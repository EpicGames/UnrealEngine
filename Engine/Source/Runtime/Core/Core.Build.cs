// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System;
using System.IO;
using System.Linq;

public class Core : ModuleRules
{
	public Core(ReadOnlyTargetRules Target) : base(Target)
	{
		bRequiresPlatformSDK = true;

		NumIncludedBytesPerUnityCPPOverride = 491520; // best unity size found from using UBT ProfileUnitySizes mode

		PrivatePCHHeaderFile = "Private/CorePrivatePCH.h";

		SharedPCHHeaderFile = "Public/CoreSharedPCH.h";

		PrivateDependencyModuleNames.Add("BuildSettings");
		if (Target.Platform == UnrealTargetPlatform.Android)
		{
			PrivateDependencyModuleNames.Add("GoogleGameSDK");
			PrivateDependencyModuleNames.Add("VKQuality");

			PrivateIncludePathModuleNames.Add("VulkanRHI"); 				// Required for AndroidPlatformMisc

			if (Target.Configuration != UnrealTargetConfiguration.Shipping)
			{
				PrivateDependencyModuleNames.Add("libGPUCounters");        // Performance counters for ARM CPUs and ARM Mali GPUs
				PrivateDependencyModuleNames.Add("heapprofd");      // Exposes custom allocators to Google's Memory Profiler
			}
			string ModulePath = Utils.MakePathRelativeTo(ModuleDirectory, Target.RelativeEnginePath);
			AdditionalPropertiesForReceipt.Add("AndroidPlugin", Path.Combine(ModulePath, "Core_APL.xml"));
		}

		PublicDefinitions.Add(string.Format("USE_ANDROID_JNI={0}", System.Convert.ToInt32(Target.bUseAndroidJni)));

		PrivateDependencyModuleNames.AddRange([
			"AtomicQueue",
			"AutoRTFM",
			"BLAKE3",
			"OodleDataCompression",
			"xxhash",
		]);

		PublicDependencyModuleNames.Add("GuidelinesSupportLibrary");
		PublicDependencyModuleNames.Add("TraceLog");
		PublicDependencyModuleNames.Add("AtomicQueue");

		PrivateIncludePathModuleNames.AddRange([
				"DerivedDataCache",
				"TargetPlatform",
				"Json",
				"RSA"
			]);

		PublicIncludePathModuleNames.AddRange([
				"AutoRTFM",
			]);

		if (Target.bBuildEditor == true)
		{
			PrivateIncludePathModuleNames.Add("DirectoryWatcher");
			DynamicallyLoadedModuleNames.Add("DirectoryWatcher");
		}

		if (Target.Platform == UnrealTargetPlatform.Win64)
		{
			PrivateDefinitions.Add("PLATFORM_BUILDS_LIBPAS=1");
			PrivateDependencyModuleNames.Add("libpas");
		}
			
		if (Target.Platform.IsInGroup(UnrealPlatformGroup.Windows))
		{
			AddEngineThirdPartyPrivateStaticDependencies(Target,
				"IntelTBB",
				"zlib"
				);

			if (Target.WindowsPlatform.Architecture != UnrealArch.Arm64)
			{
				AddEngineThirdPartyPrivateStaticDependencies(Target,
					"IntelVTune"
					);
			}

			// We do not want the static analyzer to run on thirdparty code
			if (Target.StaticAnalyzer == StaticAnalyzer.None) 
			{
				PrivateDependencyModuleNames.Add("mimalloc");
				PrivateDefinitions.Add("PLATFORM_BUILDS_MIMALLOC=1");
			}
			
			if (Target.WindowsPlatform.bUseBundledDbgHelp)
			{
				string ArchDir = (Target.Architecture == UnrealArch.Arm64) ? "arm64/" : "";
				PublicDelayLoadDLLs.Add("DBGHELP.DLL");
				PrivateDefinitions.Add("USE_BUNDLED_DBGHELP=1");
				RuntimeDependencies.Add($"$(EngineDir)/Binaries/ThirdParty/DbgHelp/{ArchDir}dbghelp.dll");
			}
			else
			{
				PrivateDefinitions.Add("USE_BUNDLED_DBGHELP=0");
			}
			PrivateDefinitions.Add("YIELD_BETWEEN_TASKS=1");

			if (Target.WindowsPlatform.bPixProfilingEnabled && Target.Configuration != UnrealTargetConfiguration.Shipping)
			{
				PrivateDependencyModuleNames.Add("WinPixEventRuntime");
			}

			if (Target.Configuration != UnrealTargetConfiguration.Shipping && Target.Type != TargetType.Program)
			{
				PublicDefinitions.Add("UE_MEMORY_TAGS_TRACE_ENABLED=1");
			}
			
			if (Target.bCompileAgainstApplicationCore)
			{
				PrivateIncludePathModuleNames.Add("ApplicationCore");
			}
		}
		else if (Target.Platform == UnrealTargetPlatform.Mac)
		{
			AddEngineThirdPartyPrivateStaticDependencies(Target,
				"IntelTBB",
				"zlib",
				"PLCrashReporter"
				);
			PublicFrameworks.AddRange(new string[] { "Cocoa", "Carbon", "IOKit", "Security", "UniformTypeIdentifiers", "Network", "ScreenCaptureKit", "SystemConfiguration" });

			PrivateDependencyModuleNames.Add("mimalloc");
			PrivateDefinitions.Add("PLATFORM_BUILDS_MIMALLOC=1");

			if (Target.bBuildEditor == true)
			{
				string XcodeRoot = Utils.RunLocalProcessAndReturnStdOut("/usr/bin/xcode-select", "--print-path");
				PublicAdditionalLibraries.Add(XcodeRoot + "/Platforms/MacOSX.platform/Developer/SDKs/MacOSX.sdk/System/Library/PrivateFrameworks/MultitouchSupport.framework/Versions/Current/MultitouchSupport.tbd");
			}
		}
		else if (Target.IsInPlatformGroup(UnrealPlatformGroup.IOS))
		{
			AddEngineThirdPartyPrivateStaticDependencies(Target,
				"zlib"
				);
			PublicFrameworks.AddRange(new string[] { "UIKit", "Foundation", "AudioToolbox", "AVFoundation", "GameKit", "CoreVideo", "CoreMedia", "CoreGraphics", "GameController", "SystemConfiguration", "DeviceCheck", "UserNotifications", "Network" });

			if (Target.Platform == UnrealTargetPlatform.IOS)
			{
				PublicFrameworks.AddRange(new string[] { "CoreMotion", "AdSupport", "WebKit" });
				AddEngineThirdPartyPrivateStaticDependencies(Target,
					"PLCrashReporter"
					);

				PrivateIncludePathModuleNames.Add("MarketplaceKit");
			}
			if (Target.Platform == UnrealTargetPlatform.VisionOS)
			{
				PublicFrameworks.Add("CoreMotion");
			}

			if (Target.bCompileAgainstApplicationCore)
			{
				PrivateIncludePathModuleNames.Add("ApplicationCore");
			}

			// export Core symbols for embedded Dlls
			ModuleSymbolVisibility = ModuleRules.SymbolVisibility.VisibileForDll;
			
			if (Target.Platform == UnrealTargetPlatform.IOS)
			{
				if (Target.Configuration != UnrealTargetConfiguration.Shipping && Target.Type != TargetType.Program)
				{
					PublicDefinitions.Add("UE_MEMORY_TRACE_AVAILABLE=1");
					PublicDefinitions.Add("UE_MEMORY_TAGS_TRACE_ENABLED=1");
					PublicDefinitions.Add("UE_CALLSTACK_TRACE_ENABLED=1");
					PublicDefinitions.Add("UE_CALLSTACK_TRACE_MAX_FRAMES=64");
				}
			}
		}
		else if (Target.IsInPlatformGroup(UnrealPlatformGroup.Android))
		{
			AddEngineThirdPartyPrivateStaticDependencies(Target,
				"cxademangle",
				"zlib",
				"libunwind"
				);

			if (Target.Configuration != UnrealTargetConfiguration.Shipping && Target.Type != TargetType.Program)
			{
				PublicDefinitions.Add("UE_MEMORY_TRACE_AVAILABLE=1");
				PublicDefinitions.Add("UE_MEMORY_TAGS_TRACE_ENABLED=1");
				PublicDefinitions.Add("UE_CALLSTACK_TRACE_ENABLED=1");
				PublicDefinitions.Add("UE_CALLSTACK_TRACE_MAX_FRAMES=64");
				PrivateDefinitions.Add("UE_CALLSTACK_TRACE_ANDROID_USE_STACK_FRAMES_WALKING=1");

				// Support for memory tracing libc.so malloc
				PrivateIncludePaths.Add(Path.Combine(EngineDirectory, "Build", "Android", "Prebuilt", "ScudoMemoryTrace"));
				PrivateDefinitions.Add("UE_MEMORY_TRACE_ANDROID_ENABLE_SCUDO_TRACING_SUPPORT=1");
			}
		}
		else if (Target.IsInPlatformGroup(UnrealPlatformGroup.Unix))
		{
			AddEngineThirdPartyPrivateStaticDependencies(Target,
				"zlib",
				"jemalloc"
				);

			// Core uses dlopen()
			PublicSystemLibraries.Add("dl");

			// Mimalloc falls foul of the memory sanitizer via startup constructors that run
			// pre main. So when using the memory sanitizer we disable use of mimalloc.
			if (!Target.LinuxPlatform.bEnableMemorySanitizer)
			{
				PrivateDependencyModuleNames.Add("mimalloc");
				PrivateDefinitions.Add("PLATFORM_BUILDS_MIMALLOC=1");
			}
			
			if (Target.Configuration != UnrealTargetConfiguration.Shipping && Target.Type != TargetType.Program)
			{
				PublicDefinitions.Add("UE_MEMORY_TRACE_AVAILABLE=1");
				PublicDefinitions.Add("UE_MEMORY_TAGS_TRACE_ENABLED=1");
				PublicDefinitions.Add("UE_CALLSTACK_TRACE_ENABLED=1");
				PrivateDefinitions.Add("UE_CALLSTACK_TRACE_UNIX_USE_STACK_FRAMES_WALKING=1");
			}
		}

		if (Target.IsInPlatformGroup(UnrealPlatformGroup.IOS) || Target.IsInPlatformGroup(UnrealPlatformGroup.Android) || Target.IsInPlatformGroup(UnrealPlatformGroup.Windows))
		{
			// For MB3 we can reduce memory retained in the allocator by limiting the size of allocations coming from bins
			// In this case if an allocation is above 14KB - it might be cached by MB3's large allocations cache, if it is enabled
			PrivateDefinitions.Add("UE_MB3_MAX_SMALL_POOL_SIZE=14336");
			PrivateDefinitions.Add("UE_MBC_MAX_LISTED_SMALL_POOL_SIZE=14336");
			PrivateDefinitions.Add("UE_MBC_NUM_LISTED_SMALL_POOLS=48");
		}

		if (Target.IsInPlatformGroup(UnrealPlatformGroup.Linux))
		{
			// Support for Linux Kernel tools
			if (Target.GlobalDefinitions.Contains("ENABLE_LINUX_UE_KERNEL_TOOLS=1"))
			{
				PrivateIncludePaths.Add(Path.Combine(EngineDirectory, "Source", "Programs", "LinuxUEKernelTools"));
			}
		}

		if (Target.IsInPlatformGroup(UnrealPlatformGroup.IOS) || Target.IsInPlatformGroup(UnrealPlatformGroup.Android))
		{
			PrivateDependencyModuleNames.Add("mimalloc212");
			PrivateDefinitions.Add("PLATFORM_BUILDS_MIMALLOC=1");
		}

		if (Target.bCompileICU == true)
		{
			AddEngineThirdPartyPrivateStaticDependencies(Target, "ICU");
		}
		PublicDefinitions.Add("UE_ENABLE_ICU=" + (Target.bCompileICU ? "1" : "0")); // Enable/disable (=1/=0) ICU usage in the codebase. NOTE: This flag is for use while integrating ICU and will be removed afterward.

		PublicDefinitions.Add($"UE_ENABLE_CONFIG_SYSTEM={(Target.bEnableConfigSystem ? 1 : 0)}");

		// If we're compiling with the engine, then add Core's engine dependencies
		if (Target.bCompileAgainstEngine && !Target.bBuildRequiresCookedData)
		{
			DynamicallyLoadedModuleNames.AddRange(new string[] { "DerivedDataCache" });
			DynamicallyLoadedModuleNames.AddRange(new string[] { "Virtualization" });
		}

		if (Target.Platform.IsInGroup(UnrealPlatformGroup.Desktop))
        {
			// Enabling more crash information on Desktop platforms.
			PublicDefinitions.Add("WITH_ADDITIONAL_CRASH_CONTEXTS=1");
		}

		if (Target.Platform.IsInGroup(UnrealPlatformGroup.Windows) && Target.Configuration != UnrealTargetConfiguration.Shipping)
		{
			// Concurrency visualizer extension for visual studio
			PrivateDependencyModuleNames.Add("ConcurrencyVisualizer");
			
			// Super Luminal.
			PrivateDependencyModuleNames.Add("SuperLuminal");
			
			// Visual studio profiler
			PrivateDependencyModuleNames.Add("VSPerfExternalProfiler");
		}

		if (Target.IsInPlatformGroup(UnrealPlatformGroup.Apple) ||
			Target.Platform == UnrealTargetPlatform.Android)
		{
			PublicDefinitions.Add("IS_RUNNING_GAMETHREAD_ON_EXTERNAL_THREAD=1");
		}

		// Set a macro to allow FApp::GetBuildTargetType() to detect client targts
		if (Target.Type == TargetRules.TargetType.Client)
		{
			PrivateDefinitions.Add("IS_CLIENT_TARGET=1");
		}
		else
		{
			PrivateDefinitions.Add("IS_CLIENT_TARGET=0");
		}

		// Setup definitions to include / exclude Iris modifications to UObject Note: Only the definition is required as we do not depend on Iris in any way.
		// To be removed once UE_WITH_IRIS has been removed in code.
		PublicDefinitions.Add("UE_WITH_IRIS=1");
		
		if (Target.Platform == UnrealTargetPlatform.Win64
			&& Target.Configuration != UnrealTargetConfiguration.Shipping)
		{
			PublicDefinitions.Add("PLATFORM_SUPPORTS_PLATFORM_EVENTS=1");
			PublicDefinitions.Add("PLATFORM_SUPPORTS_TRACE_WIN32_VIRTUAL_MEMORY_HOOKS=1");
			PublicDefinitions.Add("PLATFORM_SUPPORTS_TRACE_WIN32_MODULE_DIAGNOSTICS=1");
			PublicDefinitions.Add("PLATFORM_SUPPORTS_TRACE_WIN32_CALLSTACK=1");
			PublicDefinitions.Add("UE_MEMORY_TRACE_AVAILABLE=1");
		}

		// temporary thing.
		PrivateDefinitions.Add("PLATFORM_SUPPORTS_BINARYCONFIG=" + (SupportsBinaryConfig(Target) ? "1" : "0"));

		PublicDefinitions.Add("WITH_MALLOC_STOMP=" + (bWithMallocStomp ? "1" : "0"));

		PrivateDefinitions.Add("PLATFORM_COMPILER_OPTIMIZATION_LTCG=" + (Target.bAllowLTCG ? "1" : "0"));
		PrivateDefinitions.Add("PLATFORM_COMPILER_OPTIMIZATION_PG=" + (Target.bPGOOptimize ? "1" : "0"));
		PrivateDefinitions.Add("PLATFORM_COMPILER_OPTIMIZATION_PG_PROFILING=" + (Target.bPGOProfile ? "1" : "0"));

		PrivateDefinitions.Add("UE_DEFINE_LEGACY_MATH_CONSTANT_MACRO_NAMES=0");

		PrivateDependencyModuleNames.Add("AutoRTFM");

		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;

		IWYUSupport = IWYUSupport.KeepAsIs;

		if (Target.Platform.IsInGroup(UnrealPlatformGroup.Windows))
		{
			// WindowsMemoryTrace.cpp uses Detours in all non-shipping builds. bEnableInstrumentation is used by
			// RaceDetectorWindows.cpp, and is allowed in Shipping builds.
			if (Target.Configuration != UnrealTargetConfiguration.Shipping || Target.WindowsPlatform.bEnableInstrumentation)
			{
				PrivateDependencyModuleNames.Add("Detours");
			}
		}

		PublicDefinitions.Add("UE_DELEGATE_CHECK_LIFETIME=" + (Target.bCheckDelegateLifetime ? "1" : "0"));
		
		if (Target.IsInPlatformGroup(UnrealPlatformGroup.Apple) && Target.Platform != UnrealTargetPlatform.TVOS)
		{
			bool bEnableContactsManager = 
				Target.Platform == UnrealTargetPlatform.Mac ? (Target.bBuildEditor || Target.MacPlatform.bEnableContactsManager) : 
				Target.IOSPlatform.bEnableContactsManager;

			if(bEnableContactsManager)
			{
				PublicFrameworks.Add("Contacts");
			}

			PublicDefinitions.Add("WITH_APPLE_CONTACTS=" + (bEnableContactsManager ? "1" : "0"));
		}
	}

	protected virtual bool SupportsBinaryConfig(ReadOnlyTargetRules Target)
	{
		return true;// Target.Platform != UnrealTargetPlatform.Android;
	}

	// Decide if validating memory allocator (aka MallocStomp) can be used on the current plVatform.
	// Run-time validation must be enabled through '-stompmalloc' command line argument.
	protected virtual bool bWithMallocStomp
	{
		get => 
			Target.Configuration != UnrealTargetConfiguration.Shipping &&
			(Target.Platform == UnrealTargetPlatform.Mac ||
			 Target.Platform == UnrealTargetPlatform.Linux ||
			 Target.Platform == UnrealTargetPlatform.LinuxArm64 ||
			 Target.Platform == UnrealTargetPlatform.Win64);
	}
}
