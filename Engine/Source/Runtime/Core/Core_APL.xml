<?xml version="1.0" encoding="utf-8"?>
<!--Core module additions-->
<root xmlns:android="http://schemas.android.com/apk/res/android">
	<!-- init section is always evaluated once per architecture -->
	<init>
		<log text="Core UPL init"/>
		<!-- we cant set bAndroidProfilerAPIAvailable $S(CompileSDK) -->
	</init>

	<gameActivityImportAdditions>
		<setBoolIsGreaterEqual result="bAndroidProfilerAPIAvailable" arg1="$S(CompileSDK)" arg2="35"/>
		<if condition="bAndroidProfilerAPIAvailable">
			<true>
				<log text="Compile SDK is 35+, adding Android Profiler access."/>
				<insert>
					import com.epicgames.unreal.androidprofiling.ProfilerAccessor;
				</insert>
			</true>
			<false>
				<log text="Compile SDK is below 35, Android Profiler access will not available."/>
			</false>
		</if>
	</gameActivityImportAdditions>
	
	<gameActivityOnCreateAdditions>
		<setBoolIsGreaterEqual result="bAndroidProfilerAPIAvailable" arg1="$S(CompileSDK)" arg2="35"/>
		<if condition="bAndroidProfilerAPIAvailable">
			<true>
				<insert>
					<![CDATA[
					ProfilerAccessor.Init(this);
					]]>
				</insert>
			</true>

		</if>
	</gameActivityOnCreateAdditions>

	<proguardAdditions>
		<setBoolIsGreaterEqual result="bAndroidProfilerAPIAvailable" arg1="$S(CompileSDK)" arg2="35"/>
		<if condition="bAndroidProfilerAPIAvailable">
			<true>
				<insert>
					-keep class com.epicgames.unreal.androidprofiling.ProfilerAccessor {
					public *;
					}
				</insert>
			</true>
		</if>
	</proguardAdditions>

	<resourceCopies>
		<setBoolIsGreaterEqual result="bAndroidProfilerAPIAvailable" arg1="$S(CompileSDK)" arg2="35"/>
		<if condition="bAndroidProfilerAPIAvailable">
			<true>
				<copyFile src="$S(EngineDir)/Source/Runtime/Core/Private/Android/Java/ProfilerAccessor.java" dst="$S(BuildDir)/src/com/epicgames/unreal/androidprofiling/ProfilerAccessor.java" />
			</true>
		</if>
	</resourceCopies>
	
	<buildGradleAdditions>
		<setBoolIsGreaterEqual result="bAndroidProfilerAPIAvailable" arg1="$S(CompileSDK)" arg2="35"/>
		<if condition="bAndroidProfilerAPIAvailable">
			<true>
				<insert>
				dependencies {
					implementation('androidx.core:core:1.15.0')
				}
				</insert>
			</true>
		</if>
	</buildGradleAdditions>
</root>
