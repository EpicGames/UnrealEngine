// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "Misc/Build.h"
#include "Misc/Parse.h"
#include "Misc/CommandLine.h"

#if WITH_EDITOR
extern CORE_API bool PRIVATE_GIsRunningHybridCookedEditor;
extern CORE_API bool PRIVATE_GHasInitializedHybridCookedEditor;
#endif

/**
 * Check if the editor is running in Hybrid Cooked Editor mode (where it can dynamically choose between cooked assets and uncooked files
 */
FORCEINLINE bool IsRunningHybridCookedEditor()
{
#if WITH_EDITOR
	if (!PRIVATE_GHasInitializedHybridCookedEditor)
	{
		PRIVATE_GIsRunningHybridCookedEditor = FParse::Param(FCommandLine::Get(), TEXT("hybridcookededitor"));
		PRIVATE_GHasInitializedHybridCookedEditor = true;
	}
	return PRIVATE_GIsRunningHybridCookedEditor;
#else
	return false;
#endif
}
