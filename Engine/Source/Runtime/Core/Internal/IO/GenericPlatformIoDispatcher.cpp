// Copyright Epic Games, Inc. All Rights Reserved.

#include "IO/GenericPlatformIoDispatcher.h"

#include "Async/UniqueLock.h"
#include "HAL/Event.h"
#include "HAL/PlatformFile.h"
#include "HAL/PlatformFileManager.h"
#include "IO/IoDispatcherConfig.h"
#include "IO/PlatformIoDispatcher.h"
#include "IO/PlatformIoDispatcherBase.h"

namespace UE
{

////////////////////////////////////////////////////////////////////////////////
class FGenericPlatformIoDispatcher final
	: public FPlatformIoDispatcherBase
{
	struct FFileHandle
	{
		bool IsValid() const { return Handle.IsValid(); }

		TUniquePtr<IFileHandle> Handle;
		int64					FileSize = -1;
		uint32					CompressionBlockSize = 0;
		uint32					FileId = 0;
	};

public:
										FGenericPlatformIoDispatcher(FPlatformIoDispatcherCreateParams&& Params);
										~FGenericPlatformIoDispatcher();

	// IIoPlatformFile
	virtual TIoStatusOr<FIoFileHandle>	OpenFile(const TCHAR* Filename, const FIoFileProperties& FileProperties, FIoFileStat* OutStat = nullptr) override;
	virtual FIoStatus					CloseFile(FIoFileHandle FileHandle) override;

private:
	// FPlatformIoDispatcherBase
	virtual FIoStatus					OnInitialize() override final;
	virtual bool						Tick() override final;
	virtual uint32						OnIoThreadEntry() override final;
	virtual void						OnWakeUp() override final;
	virtual FIoPlatformFileInfo			GetPlatformFileInfo(FIoFileHandle FileHandle) override final;

	bool								ProcessFileIo(bool& bOutWaitingForMemory);
	FEventRef							WakeUpEvent;
};

////////////////////////////////////////////////////////////////////////////////
FGenericPlatformIoDispatcher::FGenericPlatformIoDispatcher(FPlatformIoDispatcherCreateParams&& Params)
	: FPlatformIoDispatcherBase(MoveTemp(Params))
{
}

FGenericPlatformIoDispatcher::~FGenericPlatformIoDispatcher()
{
	bStopRequested = true;
	if (Thread.IsValid())
	{
		WakeUpEvent->Trigger();
		Thread.Reset();
	}
}

TIoStatusOr<FIoFileHandle> FGenericPlatformIoDispatcher::OpenFile(const TCHAR* Filename, const FIoFileProperties& FileProperties, FIoFileStat* OutStat)
{
	IPlatformFile& Ipf = IPlatformFile::GetPlatformPhysical();

	TUniquePtr<FFileHandle> FileHandle	= MakeUnique<FFileHandle>();
	FileHandle->FileSize				= Ipf.FileSize(Filename);
	FileHandle->CompressionBlockSize	= FileProperties.CompressionBlockSize;
	FileHandle->FileId					= GetNextFileId();
	check(FileHandle->FileId > 0);

	if (FileHandle->FileSize < 0)
	{
		return FIoStatus(EIoErrorCode::NotFound);
	}

	FileHandle->Handle.Reset(Ipf.OpenReadNoBuffering(Filename));
	if (FileHandle.IsValid() == false)
	{
		return FIoStatus(EIoErrorCode::NotFound);
	}

	if (OutStat != nullptr)
	{
		OutStat->FileSize = FileHandle->FileSize;
	}

	return FIoFileHandle(UPTRINT(FileHandle.Release()));
}

FIoStatus FGenericPlatformIoDispatcher::CloseFile(FIoFileHandle FileHandle)
{
	if (FileHandle.IsValid())
	{
		FFileHandle* Handle = reinterpret_cast<FFileHandle*>(FileHandle.Value());
		delete Handle;
	}

	return FIoStatus::Ok;
}

FIoStatus FGenericPlatformIoDispatcher::OnInitialize()
{
	FileBlockSize								= (GIoDispatcherBufferSizeKB > 0 ? uint32(GIoDispatcherBufferSizeKB) << 10 : 256 << 10);
	const bool bSortByOffset					= GIoDispatcherSortRequestsByOffset > 0;
	const uint32 DefaultCompressionBlockSize	= 64 << 10;

	UE_LOGF(LogPlatformIoDispatcher, Log, "Initializing, Platform='%ls', ReadSize=%dKB, ReadMemory=%dMB, FileCache=%dMB, RequestSorting=%ls, MaxConcurrentDecodeJobs=%d",
		TEXT("Generic"), GIoDispatcherBufferSizeKB, GIoDispatcherBufferMemoryMB, GIoDispatcherCacheSizeMB, bSortByOffset ? TEXT("ByOffset") : TEXT("BySeqNo"), GIoDispatcherDecompressionWorkerCount);

	ChunkBlockMemoryPool.Initialize(GIoDispatcherDecompressionWorkerCount + 1, DefaultCompressionBlockSize);
	FileBlockMemoryPool.Initialize(FileBlockSize, GIoDispatcherBufferMemoryMB << 20, GIoDispatcherBufferAlignment); 
	FileBlockCache.Initialize(FileBlockSize, uint64(GIoDispatcherCacheSizeMB) << 20);
	IoQueue.SortByOffset(bSortByOffset);

	return FIoStatus::Ok;
}

bool FGenericPlatformIoDispatcher::Tick()
{
	// Only called in single threaded mode
	bool bDidSomething = false;
	for (;;)
	{
		bool bWaitingForMemory = false;
		if (ProcessFileIo(bWaitingForMemory) == false)
		{
			break;
		}
		bDidSomething = true;
	}

	return bDidSomething;
}

uint32 FGenericPlatformIoDispatcher::OnIoThreadEntry()
{
	for(;;)
	{
		bool bWaitingForMemory = false;
		const bool bDidSomething = ProcessFileIo(bWaitingForMemory);
		if (bDidSomething == false)
		{
			if (bStopRequested)
			{
				break;
			}
			{
				if (bWaitingForMemory)
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(WaitingForMemory);
					WakeUpEvent->Wait();
				}
				else
				{
					WakeUpEvent->Wait();
				}
			}
		}
	}

	return 0;
}

void FGenericPlatformIoDispatcher::OnWakeUp()
{
	WakeUpEvent->Trigger();
}

bool FGenericPlatformIoDispatcher::ProcessFileIo(bool& bOutWaitingForMemory)
{
	bOutWaitingForMemory					= false;
	FIoFileBlockRequest* FileBlockRequest	= nullptr;
	{
		UE::TUniqueLock QueueLock(IoQueue);
		if (FIoFileBlockRequest* NextPending = IoQueue.Peek(); NextPending != nullptr)
		{
			if (NextPending->ErrorCode == EIoErrorCode::Cancelled)
			{
				FileBlockRequest = IoQueue.Dequeue();
			}
			else
			{
				NextPending->Buffer = FileBlockMemoryPool.Alloc(NextPending->BufferHandle);
				if (NextPending->BufferHandle.IsValid())
				{
					FileBlockRequest = IoQueue.Dequeue();
				}
				else
				{
					bOutWaitingForMemory = true;
				}
			}
		}
	}

	if (FileBlockRequest == nullptr)
	{
		return false;
	}
	check(FileBlockRequest->Size > 0);
	check(FileBlockRequest->EncodedBlockRequests.IsEmpty() == false);
	check(FileBlockRequest->QueueStatus == FIoFileBlockRequest::EQueueStatus::Dequeued);

	if (FileBlockRequest->ErrorCode != EIoErrorCode::Cancelled)
	{
		FileBlockRequest->ErrorCode = EIoErrorCode::Ok;
		if (FileBlockCache.Get(*FileBlockRequest) == false)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(ReadBlockFromFile);

			FFileHandle&	FileHandle = *reinterpret_cast<FFileHandle*>(FileBlockRequest->FileHandle.Value());
			int32			RetryCount = 0;

			Stats.OnFileBlockReadStarted(*FileBlockRequest);
			FileBlockRequest->ErrorCode = EIoErrorCode::ReadError;
			while (RetryCount++ < 10)
			{
				if (FileHandle.Handle->Seek(int64(FileBlockRequest->FileOffset)) == false)
				{
					UE_LOGF(LogPlatformIoDispatcher, Warning, "Failed to seek to offset %lld", int64(FileBlockRequest->FileOffset));
					continue;
				}
				if (FileHandle.Handle->Read(reinterpret_cast<uint8*>(FileBlockRequest->Buffer), int64(FileBlockRequest->Size)) == false)
				{
					UE_LOGF(LogPlatformIoDispatcher, Warning, "Failed to read %lld bytes at offset %lld", int64(FileBlockRequest->Size), int64(FileBlockRequest->FileOffset));
					continue;
				}
				FileBlockRequest->ErrorCode = EIoErrorCode::Ok;
				FileBlockCache.Put(*FileBlockRequest);
				break;
			}
			Stats.OnFileBlockReadCompleted(*FileBlockRequest);
		}
	}

	EnqueueCompletedFileBlock(*FileBlockRequest);

	return true;
}

FIoPlatformFileInfo FGenericPlatformIoDispatcher::GetPlatformFileInfo(FIoFileHandle FileHandle)
{
	if (const FFileHandle* InternalHandle = reinterpret_cast<const FFileHandle*>(FileHandle.Value()))
	{
		return FIoPlatformFileInfo
		{
			.FileSize				= InternalHandle->FileSize,
			.FileId					= InternalHandle->FileId,
			.CompressionBlockSize	= InternalHandle->CompressionBlockSize
		};
	}
	
	return FIoPlatformFileInfo{};
}

////////////////////////////////////////////////////////////////////////////////
TUniquePtr<IPlatformIoDispatcher> FGenericPlatformIoDispatcherFactory::Create(FPlatformIoDispatcherCreateParams&& Params)
{
	return MakeUnique<FGenericPlatformIoDispatcher>(MoveTemp(Params));
}

////////////////////////////////////////////////////////////////////////////////
TUniquePtr<IPlatformIoDispatcher> MakeGenericPlatformIoDispatcher(FPlatformIoDispatcherCreateParams&& Params)
{
	return MakeUnique<FGenericPlatformIoDispatcher>(MoveTemp(Params));
}

} // namespace UE
