// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/UniquePtr.h"

namespace UE
{

class FGenericPlatformIoDispatcherFactory
{
public:
	CORE_API static TUniquePtr<class IPlatformIoDispatcher> Create(struct FPlatformIoDispatcherCreateParams&&);
};

using FPlatformIoDispatcherFactory = FGenericPlatformIoDispatcherFactory;

}
