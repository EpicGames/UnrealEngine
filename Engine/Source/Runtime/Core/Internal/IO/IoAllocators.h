// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/TypeCompatibleBytes.h"
#include "Templates/UnrealTemplate.h"

template <typename T, uint16 SlabSize = 4096>
class TSingleThreadedSlabAllocator
{
private:
	struct FSlab;

	struct FElement
	{
		TTypeCompatibleBytes<T> Data;
		FSlab* Slab = nullptr;
	};

	struct FSlab
	{
		uint16 Allocated = 0;
		uint16 Freed = 0;
		FElement Elements[SlabSize];
	};

public:
	TSingleThreadedSlabAllocator()
	{
		CurrentSlab = new FSlab();
	}

	~TSingleThreadedSlabAllocator()
	{
		check(CurrentSlab->Allocated == CurrentSlab->Freed);
		delete CurrentSlab;
	}

	template <typename... ArgsType>
	T* Construct(ArgsType&&... Args)
	{
		return new(Alloc()) T(Forward<ArgsType>(Args)...);
	}

	void Destroy(T* Ptr)
	{
		Ptr->~T();
		Free(Ptr);
	}

	T* Alloc()
	{
		uint16 ElementIndex = CurrentSlab->Allocated++;
		check(ElementIndex < SlabSize);
		FElement* Element = CurrentSlab->Elements + ElementIndex;
		Element->Slab = CurrentSlab;
		if (CurrentSlab->Allocated == SlabSize)
		{
			//TRACE_CPUPROFILER_EVENT_SCOPE(AllocSlab);
			CurrentSlab = new FSlab();
		}
		return Element->Data.GetTypedPtr();
	}

	void Free(T* Ptr)
	{
		FElement* Element = reinterpret_cast<FElement*>(Ptr);
		FSlab* Slab = Element->Slab;
		if (++Slab->Freed == SlabSize)
		{
			//TRACE_CPUPROFILER_EVENT_SCOPE(FreeSlab);
			check(Slab->Freed == Slab->Allocated);
			delete Slab;
		}
	}

private:
	FSlab* CurrentSlab = nullptr;
};
