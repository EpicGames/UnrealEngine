// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/MappedFileHandle.h"
#include "Async/Mutex.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/ChunkedArray.h"
#include "Containers/Map.h"
#include "Containers/StringView.h"
#include "Containers/Utf8String.h"
#include "IO/IoChunkId.h"
#include "IO/IoStatus.h"
#include "Memory/MemoryView.h"
#include "Misc/StringBuilder.h"
#include "Misc/TVariant.h"

////////////////////////////////////////////////////////////////////////////////
struct FIoMetaDirectoryIndexEntry
{
	uint32 Name				= ~uint32(0);
	uint32 ParentEntry		= ~uint32(0);
	uint32 FirstChildEntry	= ~uint32(0);
	uint32 NextSiblingEntry	= ~uint32(0);
	uint32 FirstFileEntry	= ~uint32(0);

	CORE_API friend FArchive& operator<<(FArchive& Ar, FIoMetaDirectoryIndexEntry& Entry);
};

////////////////////////////////////////////////////////////////////////////////
struct FIoMetaFileIndexEntry
{
	uint32 Name				= ~uint32(0);
	uint32 ContainerName	= ~uint32(0);
	uint32 DirectoryEntry	= ~uint32(0);
	uint32 NextFileEntry	= ~uint32(0);

	CORE_API friend FArchive& operator<<(FArchive& Ar, FIoMetaFileIndexEntry& Entry);
};

////////////////////////////////////////////////////////////////////////////////
struct FIoMetaStringTableEntry
{
	const uint32 Offset = 0;
	const uint32 Len	= 0;
};

////////////////////////////////////////////////////////////////////////////////
struct FIoContainerMetaHeader
{
	CORE_API static const TCHAR*	FileExtension;
	static const inline uint8		MagicSequence[16] = {'C', 'O', 'N', 'T', 'A', 'I', 'N', 'E', 'R', 'M', 'E', 'T', 'A', 'H', 'D', 'R'};

	enum class EVersion : uint32
	{
		Invalid	= 0,
		Initial,

		LatestPlusOne,
		Latest = LatestPlusOne - 1
	};

	CORE_API bool IsValid() const;

	uint8	Magic[16]		= {0};
	uint32	Version			= 0;
	uint32	FileCount		= 0;
	uint32	DirectoryCount	= 0;
	uint32	StringCount		= 0;
	uint8	Pad[32]			= {0};
};
static_assert(sizeof(FIoContainerMetaHeader) == 64);

////////////////////////////////////////////////////////////////////////////////
struct FIoContainerMetaResourceView
{
	bool											IsValid() const { return Header != nullptr && Header->IsValid(); }
	inline FUtf8StringView							GetString(uint32 StringTableEntryIndex) const;
	inline FUtf8StringView							GetString(FIoMetaStringTableEntry Entry) const;

	CORE_API static FIoContainerMetaResourceView	MakeView(FMemoryView MemoryView);

	const FIoContainerMetaHeader*					Header = nullptr;
	TConstArrayView<FIoChunkId>						ChunkIds;
	TConstArrayView<uint32>							FileEntryIndices;
	TConstArrayView<FIoMetaFileIndexEntry>			FileEntries;
	TConstArrayView<FIoMetaDirectoryIndexEntry>		DirectoryEntries;
	TConstArrayView<FIoMetaStringTableEntry>		StringTableEntries;
	FMemoryView										StringTable;
};

////////////////////////////////////////////////////////////////////////////////
FUtf8StringView FIoContainerMetaResourceView::GetString(FIoMetaStringTableEntry Entry) const
{
	return FUtf8StringView(reinterpret_cast<const UTF8CHAR*>(StringTable.RightChop(Entry.Offset).GetData()), Entry.Len);
}

FUtf8StringView FIoContainerMetaResourceView::GetString(uint32 StringTableEntryIndex) const
{
	check(StringTableEntryIndex != ~uint32(0));
	return GetString(StringTableEntries[int32(StringTableEntryIndex)]);
}

////////////////////////////////////////////////////////////////////////////////
using FIoContainerMetaVisistor = TFunction<bool(const FIoChunkId& ChunkId, FUtf8StringView ContainerName, FUtf8StringView Filename)>;

////////////////////////////////////////////////////////////////////////////////
class FIoContainerMetaReader
{
	struct FMappedFileStorage
	{
		TUniquePtr<IMappedFileHandle> MappedFile;
		TUniquePtr<IMappedFileRegion> MappedRegion;
	};

	struct FMemoryStorage
	{
		TArray<uint8> Buffer;
	};

	using FStorage = TVariant<FEmptyVariantState, FMappedFileStorage, FMemoryStorage>;

public:
	CORE_API					~FIoContainerMetaReader();
	CORE_API FUtf8StringView	GetFilename(const FIoChunkId& ChunkId, FUtf8StringBuilderBase& OutFilename, FUtf8StringView& OutContainerName);
	CORE_API void				Iterate(FIoContainerMetaVisistor&& Visitor);
	CORE_API static TIoStatusOr<FIoContainerMetaReader> Load(const FString& FilePath, bool bEnableMemoryMapping = true);

	FIoContainerMetaReader& operator=(const FIoContainerMetaReader&) = delete;
	FIoContainerMetaReader&	operator=(FIoContainerMetaReader&&) = default;
	FIoContainerMetaReader(FIoContainerMetaReader&&) = default;
	FIoContainerMetaReader(const FIoContainerMetaReader&) = delete;
	
private:
	FUtf8StringView	GetFilename(uint32 File, FUtf8StringBuilderBase& OutFilename, FUtf8StringView& OutContainerName);

	FIoContainerMetaReader(FMemoryStorage&& InStorage, FIoContainerMetaResourceView InView)
		: View(InView)
	{
		Storage.Emplace<FMemoryStorage>(MoveTemp(InStorage));
	}

	FIoContainerMetaReader(FMappedFileStorage&& InStorage, FIoContainerMetaResourceView InView)
		: View(InView)
	{
		Storage.Emplace<FMappedFileStorage>(MoveTemp(InStorage));
	}

	FIoContainerMetaResourceView	View;
	FStorage						Storage;
};

////////////////////////////////////////////////////////////////////////////////
class FIoContainerMetaWriter
{
public:
	CORE_API		FIoContainerMetaWriter();
	CORE_API void	AddFile(FStringView ContainerName, const FIoChunkId& ChunkId, FStringView Filename);
	CORE_API void	Save(FArchive& Ar);
	CORE_API int64	Save(const FString& FilePath);

	FIoContainerMetaWriter(const FIoContainerMetaWriter&) = delete;
	FIoContainerMetaWriter& operator=(const FIoContainerMetaWriter&) = delete;
private:
	uint32			GetDirectory(uint32 DirectoryName, uint32 Parent);
	uint32 			CreateDirectory(FStringView DirectoryName, uint32 Parent);
	uint32 			CreateDirectoryTree(FStringView Path);
	uint32 			AddFile(FStringView FileName, FStringView ContainerName, uint32 Directory);
	uint32 			GetNameIndex(const FStringView& String);

	TMap<FUtf8StringView, uint32>		StringToIndex;
	TMap<FIoChunkId, uint32>			ChunkIdToFileEntry;
	TChunkedArray<FUtf8String>			Strings;
	TArray<FIoMetaDirectoryIndexEntry>	DirectoryEntries;
	TArray<FIoMetaFileIndexEntry>		FileEntries;
	UE::FMutex							Mutex;
};
