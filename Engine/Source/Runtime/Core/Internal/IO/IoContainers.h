// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/AssertionMacros.h"

namespace UE::Private
{

template <typename TypeTraits>
class TIntrusiveListIterator
{
public:
	using ElementType = typename TypeTraits::ElementType;

	explicit TIntrusiveListIterator(ElementType* InElement)
		: Element(InElement)
	{ }

	ElementType&	operator*() const	{ check(Element); return *Element; }
	explicit		operator bool()		{ return Element != nullptr; }
	void			operator++()		{ check(Element); Element = TypeTraits::GetNext(Element); }
	bool			operator!=(const TIntrusiveListIterator& Other) const { return Element != Other.Element; }

private:
	ElementType* Element;
};

template <typename Type, typename TypeTraits>
class TIntrusiveTwoWayListIterator
{
public:
	using ElementType = Type;

	explicit TIntrusiveTwoWayListIterator(ElementType* InElement)
		: Element(InElement)
	{ }

	ElementType&	operator*() const	{ check(Element); return *Element; }
	explicit		operator bool()		{ return Element != nullptr; }
	void			operator++()		{ check(Element); Element = TypeTraits::GetNext(Element); }
	bool			operator!=(const TIntrusiveTwoWayListIterator& Other) const { return Element != Other.Element; }

private:
	ElementType* Element;
};

} // namespace UE::Private

template <typename Type>
struct TIntrusiveListElement
{
	using ElementType = Type; 

	static Type* GetNext(const ElementType* Element)
	{
		return Element->Next;
	}

	static void SetNext(ElementType* Element, ElementType* Next)
	{
		Element->Next = Next;
	}
};

template <typename TypeTraits>
class TIntrusiveList
{
public:
	using ElementType		= typename TypeTraits::ElementType;
	using FIterator			= UE::Private::TIntrusiveListIterator<TypeTraits>;
	using FConstIterator	= UE::Private::TIntrusiveListIterator<const TypeTraits>;

	TIntrusiveList() = default;
	TIntrusiveList(const TIntrusiveList&) = delete;
	TIntrusiveList(TIntrusiveList&& Other)
		: Head(Other.Head)
		, Tail(Other.Tail)
	{
		Other.Head = Other.Tail = nullptr;
	}
	explicit TIntrusiveList(ElementType* Element)
	{
		Head = Tail = Element;
	}

	TIntrusiveList& operator=(const TIntrusiveList&) = delete;
	TIntrusiveList& operator=(TIntrusiveList&& Other)
	{
		Head = Other.Head;
		Tail = Other.Tail;
		Other.Head = Other.Tail = nullptr;
		return *this;
	}

	void AddHead(ElementType* Element)
	{
		check(Element != nullptr && TypeTraits::GetNext(Element) == nullptr);

		if (Tail == nullptr)
		{
			check(Head == nullptr);
			Tail = Element;
		}
		else
		{
			TypeTraits::SetNext(Element, Head);
		}
		Head = Element;
	}

	void AddTail(ElementType* Element)
	{
		check(Element != nullptr && TypeTraits::GetNext(Element) == nullptr);

		if (Tail != nullptr)
		{
			check(Head != nullptr);
			TypeTraits::SetNext(Tail, Element);
			Tail = Element;
		}
		else
		{
			check(Head == nullptr);
			Head = Tail = Element;
		}
	}

	void AddTail(ElementType* First, ElementType* Last)
	{
		check(First && Last);
		check(TypeTraits::GetNext(First) != nullptr || First == Last);

		if (Tail != nullptr)
		{
			check(Head != nullptr);
			TypeTraits::SetNext(Tail, First);
			Tail = Last;
		}
		else
		{
			check(Head == nullptr);
			Head = First;
			Tail = Last;
		}
	}

	void AddTail(TIntrusiveList&& Other)
	{
		if (!Other.IsEmpty())
		{
			AddTail(Other.Head, Other.Tail);
			Other.Head = Other.Tail = nullptr;
		}
	}

	ElementType* PopHead()
	{
		ElementType* Element = Head;
		if (Element != nullptr)
		{
			Head = TypeTraits::GetNext(Element);
			if (Head == nullptr)
			{
				Tail = nullptr;
			}
			TypeTraits::SetNext(Element, nullptr);
		}

		return Element;
	}

	ElementType* PeekHead()
	{
		return Head;
	}

	const ElementType* PeekHead() const
	{
		return Head;
	}

	bool Remove(ElementType* Element)
	{
		if (Element == nullptr || IsEmpty())
		{
			return false;
		}

		if (Element == Head)
		{
			PopHead();
			return true;
		}

		ElementType* It = Head;
		ElementType* NextElement = TypeTraits::GetNext(It);
		while (NextElement != nullptr && NextElement != Element)
		{
			It = NextElement;
			NextElement = TypeTraits::GetNext(It);
		}

		if (NextElement != Element)
		{
			return false;
		}

		It->Next = TypeTraits::GetNext(Element);
		TypeTraits::SetNext(Element, nullptr);
		if (Element == Tail)
		{
			Tail = It;
		}

		return true;
	}

	/**
	 * Adds or inserts a new item before the existing item returned by the user defined predicate.
	 *
	 * @param Element	The new item to add
	 * @param Predicate	User defined predicate that should return true to insert the new item before the existing item
	 */
	template<typename PredicateType>
	void AddOrInsertBefore(ElementType* Element, PredicateType Predicate)
	{
		check(Element != nullptr && TypeTraits::GetNext(Element) == nullptr);

		if (IsEmpty() || Predicate(*Element, *Head))
		{
			AddHead(Element);
			return;
		}

		ElementType* It = Head;
		ElementType* NextElement = TypeTraits::GetNext(It);
		while (NextElement != nullptr)
		{
			if (Predicate(*Element, *NextElement))
			{
				TypeTraits::SetNext(It, Element);
				TypeTraits::SetNext(Element, NextElement);
				return;
			}

			It = NextElement;
			NextElement = TypeTraits::GetNext(It);
		}

		AddTail(Element);
	}

	bool				IsEmpty() const { return Head == nullptr; }
	ElementType*		GetHead()		{ return Head; }
	const ElementType*	GetHead() const { return Head; }
	ElementType*		GetTail()		{ return Tail; }
	const ElementType*	GetTail() const { return Tail; }

	FIterator			begin()			{ return FIterator(Head); }
	FConstIterator		begin() const	{ return FConstIterator(Head); }
	FIterator			end()			{ return FIterator(nullptr); }
	FConstIterator		end() const		{ return FConstIterator(nullptr); }

private:
	ElementType* Head = nullptr;
	ElementType* Tail = nullptr;
};

template <typename Type>
struct TIntrusiveTwoWayListTraits
{
	using ElementType = Type; 

	static Type* GetNext(const ElementType* Element)
	{
		return Element->Next;
	}

	static void SetNext(ElementType* Element, ElementType* Next)
	{
		Element->Next = Next;
	}

	static Type* GetPrev(const ElementType* Element)
	{
		return Element->Prev;
	}

	static void SetPrev(ElementType* Element, ElementType* Prev)
	{
		Element->Prev = Prev;
	}
};

template <typename ElementType, typename TypeTraits = TIntrusiveTwoWayListTraits<ElementType>>
class TIntrusiveTwoWayList
{
public:

	using FIterator			= UE::Private::TIntrusiveTwoWayListIterator<ElementType, TypeTraits>;
	using FConstIterator	= UE::Private::TIntrusiveTwoWayListIterator<ElementType, const TypeTraits>;

	TIntrusiveTwoWayList() = default;
	TIntrusiveTwoWayList(const TIntrusiveTwoWayList&) = delete;
	TIntrusiveTwoWayList(TIntrusiveTwoWayList&& Other)
		: Head(Other.Head)
		, Tail(Other.Tail)
	{
		Other.Head = Other.Tail = nullptr;
	}
	explicit TIntrusiveTwoWayList(ElementType* Element)
	{
		Head = Tail = Element;
	}

	TIntrusiveTwoWayList& operator=(const TIntrusiveTwoWayList&) = delete;
	TIntrusiveTwoWayList& operator=(TIntrusiveTwoWayList&& Other)
	{
		Head = Other.Head;
		Tail = Other.Tail;
		Other.Head = Other.Tail = nullptr;
		return *this;
	}

	void AddTail(ElementType* Element)
	{
		check(Element != nullptr && TypeTraits::GetNext(Element) == nullptr && TypeTraits::GetPrev(Element) == nullptr);

		if (Tail != nullptr)
		{
			check(Head != nullptr);
			TypeTraits::SetNext(Tail, Element);
			TypeTraits::SetPrev(Element, Tail);
			Tail = Element;
		}
		else
		{
			check(Head == nullptr);
			Head = Tail = Element;
		}
	}

	void AddHead(ElementType* Element)
	{
		check(Element != nullptr && TypeTraits::GetNext(Element) == nullptr && TypeTraits::GetPrev(Element) == nullptr);

		if (Head != nullptr)
		{
			check(Tail != nullptr);
			TypeTraits::SetNext(Element, Head);
			TypeTraits::SetPrev(Head, Element);
			Head = Element;
		}
		else
		{
			check(Tail == nullptr);
			Head = Tail = Element;
		}
	}

	[[nodiscard]] ElementType* PopHead()
	{
		ElementType* Element = Head;
		if (Element != nullptr)
		{
			Head = TypeTraits::GetNext(Element);
			TypeTraits::SetNext(Element, nullptr);

			if (Head != nullptr)
			{
				TypeTraits::SetPrev(Head, nullptr);
			}
			else
			{
				Tail = nullptr;
			}
		}

		return Element;
	}

	[[nodiscard]] ElementType* PeekHead()
	{
		return Head;
	}

	bool				IsEmpty() const { return Head == nullptr; }
	ElementType*		GetHead()		{ return Head; }
	const ElementType*	GetHead() const { return Head; }
	ElementType*		GetTail()		{ return Tail; }
	const ElementType*	GetTail() const { return Tail; }

	FIterator			begin()			{ return FIterator(Head); }
	FConstIterator		begin() const	{ return FConstIterator(Head); }
	FIterator			end()			{ return FIterator(nullptr); }
	FConstIterator		end() const		{ return FConstIterator(nullptr); }

	void Remove(ElementType* Element)
	{
		check(Element != nullptr);

		if (Head == Element && Tail == Element)
		{
			check(TypeTraits::GetNext(Element) == nullptr);
			check(TypeTraits::GetPrev(Element) == nullptr);

			Head = Tail = nullptr;
		}
		else if (Head == Element)
		{
			check(TypeTraits::GetPrev(Element) == nullptr);

			Head = TypeTraits::GetNext(Element);
			TypeTraits::SetPrev(Head, nullptr);
			TypeTraits::SetNext(Element, nullptr);
		}
		else if (Tail == Element)
		{
			check(Element->Next == nullptr);

			Tail = TypeTraits::GetPrev(Element);
			TypeTraits::SetNext(Tail, nullptr);
			TypeTraits::SetPrev(Element, nullptr);
		}
		else
		{
			ElementType* NextElement = TypeTraits::GetNext(Element);
			ElementType* PrevElement = TypeTraits::GetPrev(Element);

			TypeTraits::SetPrev(NextElement, PrevElement);
			TypeTraits::SetNext(PrevElement, NextElement);

			TypeTraits::SetNext(Element, nullptr);
			TypeTraits::SetPrev(Element, nullptr);
		}
	}

private:
	ElementType* Head = nullptr;
	ElementType* Tail = nullptr;
};
