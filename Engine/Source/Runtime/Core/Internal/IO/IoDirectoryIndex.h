// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ChunkedArray.h"
#include "Containers/Map.h"
#include "Containers/StringFwd.h"
#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "Misc/AES.h"

class FArchive;

namespace IoDirectoryIndexUtils
{
	CORE_API FString GetCommonRootPath(const TArray<FStringView>& Filenames);
}

struct FIoDirectoryIndexEntry
{
	uint32 Name				= ~uint32(0);
	uint32 FirstChildEntry	= ~uint32(0);
	uint32 NextSiblingEntry	= ~uint32(0);
	uint32 FirstFileEntry	= ~uint32(0);

	friend FArchive& operator<<(FArchive& Ar, FIoDirectoryIndexEntry& Entry);
};

struct FIoFileIndexEntry
{
	uint32 Name				= ~uint32(0);
	uint32 NextFileEntry	= ~uint32(0);
	uint32 UserData			= 0;

	friend FArchive& operator<<(FArchive& Ar, FIoFileIndexEntry& Entry);
};

struct FIoDirectoryIndexResource
{
	FString MountPoint;
	TArray<FIoDirectoryIndexEntry> DirectoryEntries;
	TArray<FIoFileIndexEntry> FileEntries;
	TArray<FString> StringTable;

	friend FArchive& operator<<(FArchive& Ar, FIoDirectoryIndexResource& Entry);
};

class FIoDirectoryIndexWriter
{
public:
	CORE_API FIoDirectoryIndexWriter();

	CORE_API void SetMountPoint(FString InMountPoint);
	CORE_API uint32 AddFile(const FStringView& InFileName);
	CORE_API void SetFileUserData(uint32 InFileEntryIndex, uint32 InUserData);

	// Flush() can be called only once, because afterwards contents of Strings member
	// will be empty as all strings have been moved away
	CORE_API void Flush(TArray<uint8>& OutBuffer, FAES::FAESKey InEncryptionKey);

private:
	uint32 GetDirectory(uint32 DirectoryName, uint32 Parent);
	uint32 CreateDirectory(const FStringView& DirectoryName, uint32 Parent);
	uint32 GetNameIndex(const FStringView& String);
	uint32 AddFile(const FStringView& FileName, uint32 Directory);

	static bool IsValid(uint32 Index)
	{
		return Index != ~uint32(0);
	}

	FString MountPoint;
	TArray<FIoDirectoryIndexEntry> DirectoryEntries;
	TArray<FIoFileIndexEntry> FileEntries;

	// These FStringView's are referencing stable FString locations inside Strings
	// member, and Strings member is only appended, never removed. It is safe to
	// use StringToIndex during lifetime of this object while Strings member is
	// populated by AddFile() until Flush() is called
	TMap<FStringView, uint32> StringToIndex;
	TChunkedArray<FString> Strings;
};
