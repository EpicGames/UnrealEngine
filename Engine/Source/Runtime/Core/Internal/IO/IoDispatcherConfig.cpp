// Copyright Epic Games, Inc. All Rights Reserved.

#include "IO/IoDispatcherConfig.h"
#include "Misc/CommandLine.h"
#include "HAL/IConsoleManager.h"
#include "Misc/Parse.h"

#ifndef UE_PLATFORM_IO_DISPATCHER_ENABLED
#define UE_PLATFORM_IO_DISPATCHER_ENABLED 0
#endif

int32 GIoDispatcherBufferSizeKB = 256;
static FAutoConsoleVariableRef CVar_IoDispatcherBufferSizeKB(
	TEXT("s.IoDispatcherBufferSizeKB"),
	GIoDispatcherBufferSizeKB,
	TEXT("IoDispatcher read buffer size (in kilobytes).")
);

int32 GIoDispatcherBufferAlignment = 4096;
static FAutoConsoleVariableRef CVar_IoDispatcherBufferAlignment(
	TEXT("s.IoDispatcherBufferAlignment"),
	GIoDispatcherBufferAlignment,
	TEXT("IoDispatcher read buffer alignment.")
);

int32 GIoDispatcherBufferMemoryMB = 8;
static FAutoConsoleVariableRef CVar_IoDispatcherBufferMemoryMB(
	TEXT("s.IoDispatcherBufferMemoryMB"),
	GIoDispatcherBufferMemoryMB,
	TEXT("IoDispatcher buffer memory size (in megabytes).")
);

int32 GIoDispatcherDecompressionWorkerCount = 4;
static FAutoConsoleVariableRef CVar_IoDispatcherDecompressionWorkerCount(
	TEXT("s.IoDispatcherDecompressionWorkerCount"),
	GIoDispatcherDecompressionWorkerCount,
	TEXT("IoDispatcher decompression worker count.")
);

int32 GIoDispatcherCacheSizeMB = 0;
static FAutoConsoleVariableRef CVar_IoDispatcherCacheSizeMB(
	TEXT("s.IoDispatcherCacheSizeMB"),
	GIoDispatcherCacheSizeMB,
	TEXT("IoDispatcher cache memory size (in megabytes).")
);

int32 GIoDispatcherSortRequestsByOffset = 1;
static FAutoConsoleVariableRef CVar_IoDispatcherSortRequestsByOffset(
	TEXT("s.IoDispatcherSortRequestsByOffset"),
	GIoDispatcherSortRequestsByOffset,
	TEXT("If > 0, io dispatcher sorts the outstanding request queue by offset rather than sequence.")
);

int32 GIoDispatcherMaintainSortingOnPriorityChange = 1;
static FAutoConsoleVariableRef CVar_IoDispatcherMaintainSortingOnPriorityChange(
	TEXT("s.IoDispatcherMaintainSortingOnPriorityChange"),
	GIoDispatcherMaintainSortingOnPriorityChange,
	TEXT("If s.IoDispatcherSortRequestsByOffset > 0 and this > 0, io dispatcher remembers the last file handle/offset read from even when switching priority levels.")
);

int32 GIoDispatcherMaxForwardSeekKB = 0;
static FAutoConsoleVariableRef CVar_IoDispatcherMaxForwardSeekKB(
	TEXT("s.IoDispatcherMaxForwardSeekKB"),
	GIoDispatcherMaxForwardSeekKB,
	TEXT("If s.IoDispatcherSortRequestsByOffset is enabled and this is > 0, if the next sequential read is further than this offset from the last one, read the oldest request instead")
);

int32 GIoDispatcherRequestLatencyCircuitBreakerMS = 0;
static FAutoConsoleVariableRef CVar_IoDispatcherRequestLatencyCircuitBreakerMS(
	TEXT("s.IoDispatcherRequestLatencyCircuitBreakerMS"),
	GIoDispatcherRequestLatencyCircuitBreakerMS,
	TEXT("If s.IoDispatcherSortRequestsByOffset is enabled and this is >0, if the oldest request has been in the queue for this long, read it instead of the most optimal read")
);

int32 GIoDispatcherTocsEnablePerfectHashing = 1;
static FAutoConsoleVariableRef CVar_IoDispatcherTocsEnablePerfectHashing(
	TEXT("s.IoDispatcherTocsEnablePerfectHashing"),
	GIoDispatcherTocsEnablePerfectHashing,
	TEXT("Enable perfect hashmap lookups for iostore tocs")
);

int32 GIoDispatcherCanDecompressOnStarvation = 1;
static FAutoConsoleVariableRef CVar_IoDispatcherCanDecompressOnStarvation(
	TEXT("s.IoDispatcherCanDecompressOnStarvation"),
	GIoDispatcherCanDecompressOnStarvation,
	TEXT("IoDispatcher thread will help with decompression tasks when all worker threads are IO starved to avoid deadlocks on low core count")
);

int32 GIoDispatcherForceSynchronousScatter = 0;
static FAutoConsoleVariableRef CVar_IoDispatcherForceSynchronousScatter(
	TEXT("s.IoDispatcherForceSynchronousScatter"),
	GIoDispatcherForceSynchronousScatter,
	TEXT("Force scatter jobs to be synchronous on the IODispatcher thread.\n")
	TEXT("This can avoid deadlocks in cases where background tasks end up waiting on I/O and we don't have enough background task threads to fulfill decompression requests.")
);

int32 GIoDispatcherDecompressOnIdle = 0;
static FAutoConsoleVariableRef CVar_GIoDispatcherDecompressOnIdle(
	TEXT("s.IoDispatcherDecompressOnIdle"),
	GIoDispatcherDecompressOnIdle,
	TEXT("When enabled the I/O dispatcher thread will help out decompress chunk blocks when idle.")
);

int32 GIoDispatcherMaxConsecutiveDecompressionJobs = 4;
static FAutoConsoleVariableRef CVar_IoDispatcherMaxConsecutiveDecompressionJobs(
	TEXT("s.IoDispatcherMaxConsecutiveDecompressionJobs"),
	GIoDispatcherMaxConsecutiveDecompressionJobs,
	TEXT("Max consecutive decompression jobs before re-launching tasks.")
);

int32 GIoDispatcherMaxResolveBatchSize = 512;
static FAutoConsoleVariableRef CVar_IoDispatcherMaxResolveBatchSize(
	TEXT("s.IoDispatcherMaxResolveBatchSize"),
	GIoDispatcherMaxResolveBatchSize,
	TEXT("")
);

bool IsPlatformIoDispatcherEnabled()
{
	static struct FIoDispatcherConfig
	{
		bool bPlatformIoDispatcherEnabled = false;
		FIoDispatcherConfig(const TCHAR* CmdLine)
		{
#if UE_PLATFORM_IO_DISPATCHER_ENABLED
			bPlatformIoDispatcherEnabled = true;
#endif
			if (FParse::Param(CmdLine, TEXT("NoPlatformIo")))
			{
				bPlatformIoDispatcherEnabled = false;
				return;
			}

			bPlatformIoDispatcherEnabled |= FParse::Param(CmdLine, TEXT("PlatformIo"));
			if (!bPlatformIoDispatcherEnabled)
			{
				FString Line;
				const bool bShouldStopOnSeparator = false;
				while (FParse::Value(CmdLine, TEXT("ExecCmds="), Line, bShouldStopOnSeparator, &CmdLine) && !bPlatformIoDispatcherEnabled)
				{
					bPlatformIoDispatcherEnabled = Line.Contains(TEXT("s.PlatformIo 1"), ESearchCase::IgnoreCase);
				}
			}
		}
	} IoDispatcherConfig(FCommandLine::Get());

	return IoDispatcherConfig.bPlatformIoDispatcherEnabled;
}
