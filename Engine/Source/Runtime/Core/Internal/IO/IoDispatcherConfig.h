// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

CORE_API extern int32 GIoDispatcherBufferSizeKB;
CORE_API extern int32 GIoDispatcherBufferAlignment;
CORE_API extern int32 GIoDispatcherBufferMemoryMB;
CORE_API extern int32 GIoDispatcherDecompressionWorkerCount;
CORE_API extern int32 GIoDispatcherCacheSizeMB;
CORE_API extern int32 GIoDispatcherSortRequestsByOffset;
CORE_API extern int32 GIoDispatcherMaintainSortingOnPriorityChange;
CORE_API extern int32 GIoDispatcherMaxForwardSeekKB;
CORE_API extern int32 GIoDispatcherRequestLatencyCircuitBreakerMS;
CORE_API extern int32 GIoDispatcherTocsEnablePerfectHashing;
CORE_API extern int32 GIoDispatcherCanDecompressOnStarvation;
CORE_API extern int32 GIoDispatcherForceSynchronousScatter;
CORE_API extern int32 GIoDispatcherDecompressOnIdle;
CORE_API extern int32 GIoDispatcherMaxConsecutiveDecompressionJobs;
CORE_API extern int32 GIoDispatcherMaxResolveBatchSize;

CORE_API bool IsPlatformIoDispatcherEnabled();
