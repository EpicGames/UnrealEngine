// Copyright Epic Games, Inc. All Rights Reserved.

#include "IO/IoDispatcherFilesystemStats.h"
#include "ProfilingDebugging/CsvProfiler.h"

namespace UE
{

#if UE_IO_DISPATCHER_FILESYSTEM_STATS_ENABLED 

////////////////////////////////////////////////////////////////////////////////
CSV_DEFINE_CATEGORY(IoDispatcherFileBackend, true);
CSV_DEFINE_CATEGORY(IoDispatcherFileBackendVerbose, false);

// These stats go to both insights and csv by default
// TODO: Ideally these should go to insights even if CSV is not capturing, but not be doubled-up if both CSV and Insights are capturing
// TODO: It would also be nice to send these to insights as int64 without unit conversion where appropriate
// IoDispatcher thread
CSV_DEFINE_STAT(IoDispatcherFileBackend,			FrameBytesScatteredKB);
CSV_DEFINE_STAT(IoDispatcherFileBackend,			QueuedFilesystemReadMB);
CSV_DEFINE_STAT(IoDispatcherFileBackendVerbose,	QueuedFilesystemReads);
CSV_DEFINE_STAT(IoDispatcherFileBackendVerbose,	QueuedUncompressBlocks);	
CSV_DEFINE_STAT(IoDispatcherFileBackendVerbose,	QueuedUncompressInMB);		
CSV_DEFINE_STAT(IoDispatcherFileBackendVerbose,	QueuedUncompressOutMB);	
CSV_DEFINE_STAT(IoDispatcherFileBackendVerbose,	FrameBytesReadKB);				
CSV_DEFINE_STAT(IoDispatcherFileBackendVerbose,	FrameBytesUncompressedInKB);	
CSV_DEFINE_STAT(IoDispatcherFileBackendVerbose,	FrameBytesUncompressedOutKB);	

// FileIoStore thread			
CSV_DEFINE_STAT(IoDispatcherFileBackend,			FrameFilesystemBytesReadKB);	
CSV_DEFINE_STAT(IoDispatcherFileBackend,			FrameSequentialReads);
CSV_DEFINE_STAT(IoDispatcherFileBackend,			FrameSeeks);
CSV_DEFINE_STAT(IoDispatcherFileBackendVerbose,	FrameFilesystemReads);
CSV_DEFINE_STAT(IoDispatcherFileBackendVerbose,	FrameForwardSeeks);
CSV_DEFINE_STAT(IoDispatcherFileBackendVerbose,	FrameBackwardSeeks);
CSV_DEFINE_STAT(IoDispatcherFileBackendVerbose,	FrameHandleChangeSeeks);
CSV_DEFINE_STAT(IoDispatcherFileBackendVerbose,	FrameSeekDistanceMB);	
CSV_DEFINE_STAT(IoDispatcherFileBackendVerbose,	FrameBlockCacheStores);
CSV_DEFINE_STAT(IoDispatcherFileBackendVerbose,	FrameBlockCacheStoresKB);
CSV_DEFINE_STAT(IoDispatcherFileBackendVerbose,	FrameBlockCacheHits);	
CSV_DEFINE_STAT(IoDispatcherFileBackendVerbose,	FrameBlockCacheHitKB);	
CSV_DEFINE_STAT(IoDispatcherFileBackendVerbose,	FrameBlockCacheMisses);
CSV_DEFINE_STAT(IoDispatcherFileBackendVerbose,	FrameBlockCacheMissKB);

////////////////////////////////////////////////////////////////////////////////
namespace IoDispatcherFilesystemStats
{

static float BytesToApproxMB(uint64 Bytes)
{
	return float(double(Bytes) / 1024.0 / 1024.0);
}

static float BytesToApproxKB(uint64 Bytes)
{
	return float(double(Bytes) / 1024.0);
}

} // namespace IoDispatcherFilesystemStats

////////////////////////////////////////////////////////////////////////////////
FIoDispatcherFilesystemStats::FIoDispatcherFilesystemStats()
#if COUNTERSTRACE_ENABLED
	: QueuedReadRequestsSizeCounter(TEXT("FileIoStore/QueuedReadRequestsSize"), TraceCounterDisplayHint_Memory)
	, CompletedReadRequestsSizeCounter(TEXT("FileIoStore/CompletedReadRequestsSize"), TraceCounterDisplayHint_Memory)
	, QueuedCompressedSizeCounter(TEXT("FileIoStore/QueuedCompressedSize"), TraceCounterDisplayHint_Memory)
	, QueuedUncompressedSizeCounter(TEXT("FileIoStore/QueuedUncompressedSize"), TraceCounterDisplayHint_Memory)
	, CompletedCompressedSizeCounter(TEXT("FileIoStore/CompletedCompressedSize"), TraceCounterDisplayHint_Memory)
	, CompletedUncompressedSizeCounter(TEXT("FileIoStore/CompletedUncompressedSize"), TraceCounterDisplayHint_Memory)
	, FileSystemSeeksTotalDistanceCounter(TEXT("FileIoStore/FileSystemSeeksTotalDistance"), TraceCounterDisplayHint_Memory)
	, FileSystemSeeksForwardCountCounter(TEXT("FileIoStore/FileSystemSeeksForwardCount"), TraceCounterDisplayHint_None)
	, FileSystemSeeksBackwardCountCounter(TEXT("FileIoStore/FileSystemSeeksBackwardCount"), TraceCounterDisplayHint_None)
	, FileSystemSeeksChangeHandleCountCounter(TEXT("FileIoStore/FileSystemSeeksChangeHandleCount"), TraceCounterDisplayHint_None)
	, FileSystemCompletedRequestsSizeCounter(TEXT("FileIoStore/FileSystemCompletedRequestsSize"), TraceCounterDisplayHint_Memory)
	, BlockCacheStoredSizeCounter(TEXT("FileIoStore/BlockCacheStoredSize"), TraceCounterDisplayHint_Memory)
	, BlockCacheHitSizeCounter(TEXT("FileIoStore/BlockCacheHitSize"), TraceCounterDisplayHint_Memory)
	, BlockCacheMissedSizeCounter(TEXT("FileIoStore/BlockCacheMissedSize"), TraceCounterDisplayHint_Memory)
	, ScatteredSizeCounter(TEXT("FileIoStore/ScatteredSize"), TraceCounterDisplayHint_Memory)
	, TocMemoryCounter(TEXT("FileIoStore/TocMemory"), TraceCounterDisplayHint_Memory)
	, AvailableBuffersCounter(TEXT("FileIoStore/AvailableBuffers"), TraceCounterDisplayHint_None)
#endif
{
#if CSV_PROFILER_STATS
	TickerHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &FIoDispatcherFilesystemStats::CsvTick));
#endif
}

FIoDispatcherFilesystemStats::~FIoDispatcherFilesystemStats()
{
#if CSV_PROFILER_STATS
	FTSTicker::RemoveTicker(TickerHandle);
#endif
}

bool FIoDispatcherFilesystemStats::CsvTick(float DeltaTime)
{
	using namespace IoDispatcherFilesystemStats;
#if CSV_PROFILER_STATS
	CSV_CUSTOM_STAT_DEFINED(QueuedFilesystemReadMB, BytesToApproxMB(QueuedFilesystemReadBytes.load(std::memory_order_relaxed)), ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT_DEFINED(QueuedFilesystemReads, (int32)QueuedFilesystemReads.load(std::memory_order_relaxed), ECsvCustomStatOp::Set);

	CSV_CUSTOM_STAT_DEFINED(QueuedUncompressBlocks, int32(QueuedUncompressBlocks), ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT_DEFINED(QueuedUncompressInMB, BytesToApproxMB(QueuedUncompressBytesIn), ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT_DEFINED(QueuedUncompressOutMB, BytesToApproxMB(QueuedUncompressBytesOut), ECsvCustomStatOp::Set);
#endif

	return true;
}

void FIoDispatcherFilesystemStats::OnReadRequestsQueued(uint64 ByteCount, uint64 ReadCount)
{
#if CSV_PROFILER_STATS
	QueuedFilesystemReadBytes.fetch_add(ByteCount, std::memory_order_relaxed);
	QueuedFilesystemReads.fetch_add(ReadCount, std::memory_order_relaxed);
#endif
#if COUNTERSTRACE_ENABLED
	QueuedReadRequestsSizeCounter.Add(ByteCount);
#endif
}

void FIoDispatcherFilesystemStats::OnFilesystemReadStarted(uint64 FileHandle, uint64 Offset, uint64 Size)
{
	CSV_CUSTOM_STAT_DEFINED(FrameFilesystemReads, 1, ECsvCustomStatOp::Accumulate);

	if (LastFileReadInfo.FileHandle != FileHandle) 
	{
		OnHandleChangeSeek();
	}
	else if (LastFileReadInfo.Offset == Offset)
	{
		OnSequentialRead();
	}
	else
	{
		OnSeek(LastFileReadInfo.Offset, Offset); 
	}

	LastFileReadInfo = FFileReadInfo
	{
		.FileHandle = FileHandle,
		.Offset = Offset + Size
	};
}

void FIoDispatcherFilesystemStats::OnFilesystemReadCompleted(uint64 FileHandle, uint64 Offset, uint64 Size)
{
	CSV_CUSTOM_STAT_DEFINED(FrameFilesystemBytesReadKB, IoDispatcherFilesystemStats::BytesToApproxKB(Size), ECsvCustomStatOp::Accumulate);
#if COUNTERSTRACE_ENABLED
	FileSystemCompletedRequestsSizeCounter.Add(Size);
#endif
}

void FIoDispatcherFilesystemStats::OnReadRequestsCompleted(uint64 ByteCount, uint64 ReadCount)
{
#if CSV_PROFILER_STATS
	QueuedFilesystemReadBytes -= ByteCount;
	QueuedFilesystemReads -= ReadCount;
#endif

	CSV_CUSTOM_STAT_DEFINED(FrameBytesReadKB, IoDispatcherFilesystemStats::BytesToApproxKB(ByteCount), ECsvCustomStatOp::Accumulate); // TODO: Send to insights if CSV is disabled

#if COUNTERSTRACE_ENABLED
	QueuedReadRequestsSizeCounter.Subtract(ByteCount);
	CompletedReadRequestsSizeCounter.Add(ByteCount);
#endif
}

void FIoDispatcherFilesystemStats::OnDecompressQueued(uint64 CompressedSize, uint64 UncompressedSize)
{
#if CSV_PROFILER_STATS
	++QueuedUncompressBlocks;
	QueuedUncompressBytesIn += CompressedSize;
	QueuedUncompressBytesOut += UncompressedSize;
#endif

#if COUNTERSTRACE_ENABLED
	QueuedCompressedSizeCounter.Add(CompressedSize);
	QueuedUncompressedSizeCounter.Add(UncompressedSize);
#endif
}

void FIoDispatcherFilesystemStats::OnDecompressComplete(uint64 CompressedSize, uint64 UncompressedSize)
{
	using namespace IoDispatcherFilesystemStats;
#if CSV_PROFILER_STATS
	--QueuedUncompressBlocks;
	QueuedUncompressBytesIn -= CompressedSize;
	QueuedUncompressBytesOut -= UncompressedSize;
#endif

	CSV_CUSTOM_STAT_DEFINED(FrameBytesUncompressedInKB, BytesToApproxKB(CompressedSize), ECsvCustomStatOp::Accumulate); 
	CSV_CUSTOM_STAT_DEFINED(FrameBytesUncompressedOutKB, BytesToApproxKB(UncompressedSize), ECsvCustomStatOp::Accumulate);

#if COUNTERSTRACE_ENABLED
	QueuedCompressedSizeCounter.Subtract(CompressedSize);
	QueuedUncompressedSizeCounter.Subtract(UncompressedSize);

	CompletedCompressedSizeCounter.Add(CompressedSize);
	CompletedUncompressedSizeCounter.Add(UncompressedSize);
#endif
}

void FIoDispatcherFilesystemStats::OnBytesScattered(int64 NumBytes)
{
	CSV_CUSTOM_STAT_DEFINED(FrameBytesScatteredKB, IoDispatcherFilesystemStats::BytesToApproxKB(NumBytes), ECsvCustomStatOp::Accumulate);
#if COUNTERSTRACE_ENABLED
	ScatteredSizeCounter.Add(NumBytes);
#endif
}

void FIoDispatcherFilesystemStats::OnSequentialRead()
{
	CSV_CUSTOM_STAT_DEFINED(FrameSequentialReads, 1, ECsvCustomStatOp::Accumulate);
}

void FIoDispatcherFilesystemStats::OnSeek(uint64 PrevOffset, uint64 NewOffset)
{
	using namespace IoDispatcherFilesystemStats;
	if (NewOffset > PrevOffset)
	{
		int64 Delta = NewOffset - PrevOffset;

		CSV_CUSTOM_STAT_DEFINED(FrameForwardSeeks, 1, ECsvCustomStatOp::Accumulate);
		CSV_CUSTOM_STAT_DEFINED(FrameSeekDistanceMB, BytesToApproxMB(Delta), ECsvCustomStatOp::Accumulate);

#if COUNTERSTRACE_ENABLED
		FileSystemSeeksTotalDistanceCounter.Add(Delta);
		FileSystemSeeksForwardCountCounter.Increment();
#endif
	}
	else
	{
		int64 Delta = PrevOffset - NewOffset;
		CSV_CUSTOM_STAT_DEFINED(FrameBackwardSeeks, 1, ECsvCustomStatOp::Accumulate);
		CSV_CUSTOM_STAT_DEFINED(FrameSeekDistanceMB, BytesToApproxMB(Delta), ECsvCustomStatOp::Accumulate);

#if COUNTERSTRACE_ENABLED
		FileSystemSeeksTotalDistanceCounter.Add(Delta);
		FileSystemSeeksBackwardCountCounter.Increment();
#endif
	}

	CSV_CUSTOM_STAT_DEFINED(FrameSeeks, 1, ECsvCustomStatOp::Accumulate);
}

void FIoDispatcherFilesystemStats::OnHandleChangeSeek()
{
	CSV_CUSTOM_STAT_DEFINED(FrameHandleChangeSeeks, 1, ECsvCustomStatOp::Accumulate);
	CSV_CUSTOM_STAT_DEFINED(FrameSeeks, 1, ECsvCustomStatOp::Accumulate);

#if COUNTERSTRACE_ENABLED
	FileSystemSeeksChangeHandleCountCounter.Increment();
#endif
}

void FIoDispatcherFilesystemStats::OnBlockCacheStore(uint64 NumBytes)
{
	CSV_CUSTOM_STAT_DEFINED(FrameBlockCacheStores, 1, ECsvCustomStatOp::Accumulate);
	CSV_CUSTOM_STAT_DEFINED(FrameBlockCacheStoresKB, IoDispatcherFilesystemStats::BytesToApproxKB(NumBytes), ECsvCustomStatOp::Accumulate);

#if COUNTERSTRACE_ENABLED
	BlockCacheStoredSizeCounter.Add(NumBytes);
#endif
}

void FIoDispatcherFilesystemStats::OnBlockCacheHit(uint64 NumBytes)
{
	CSV_CUSTOM_STAT_DEFINED(FrameBlockCacheHits, 1, ECsvCustomStatOp::Accumulate);
	CSV_CUSTOM_STAT_DEFINED(FrameBlockCacheHitKB, IoDispatcherFilesystemStats::BytesToApproxKB(NumBytes), ECsvCustomStatOp::Accumulate);

#if COUNTERSTRACE_ENABLED
	BlockCacheHitSizeCounter.Add(NumBytes);
#endif
}

void FIoDispatcherFilesystemStats::OnBlockCacheMiss(uint64 NumBytes)
{
	CSV_CUSTOM_STAT_DEFINED(FrameBlockCacheMisses, 1, ECsvCustomStatOp::Accumulate);
	CSV_CUSTOM_STAT_DEFINED(FrameBlockCacheMissKB, IoDispatcherFilesystemStats::BytesToApproxKB(NumBytes), ECsvCustomStatOp::Accumulate);

#if COUNTERSTRACE_ENABLED
	BlockCacheMissedSizeCounter.Add(NumBytes);
#endif
}

void FIoDispatcherFilesystemStats::OnTocMounted(uint64 AllocatedSize)
{
#if COUNTERSTRACE_ENABLED
	TocMemoryCounter.Add(AllocatedSize);
#endif
}

void FIoDispatcherFilesystemStats::OnTocUnmounted(uint64 AllocatedSize)
{
#if COUNTERSTRACE_ENABLED
	TocMemoryCounter.Subtract(AllocatedSize);
#endif
}

void FIoDispatcherFilesystemStats::OnBufferReleased()
{
#if COUNTERSTRACE_ENABLED
	AvailableBuffersCounter.Increment();
#endif
}

void FIoDispatcherFilesystemStats::OnBufferAllocated()
{
#if COUNTERSTRACE_ENABLED
	AvailableBuffersCounter.Decrement();
#endif
}

#endif // UE_IO_DISPATCHER_FILESYSTEM_STATS_ENABLED 

} // namespace UE
