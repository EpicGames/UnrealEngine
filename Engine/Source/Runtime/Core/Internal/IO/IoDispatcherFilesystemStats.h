// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Ticker.h"
#include "Math/NumericLimits.h"
#include "ProfilingDebugging/CountersTrace.h"

#define UE_IO_DISPATCHER_FILESYSTEM_STATS_ENABLED (COUNTERSTRACE_ENABLED || CSV_PROFILER_STATS)

namespace UE
{

#if UE_IO_DISPATCHER_FILESYSTEM_STATS_ENABLED 
class FIoDispatcherFilesystemStats
{
	struct FFileReadInfo
	{
		uint64	FileHandle	= 0;
		uint64	Offset		= MAX_uint64;
	};

public:
	CORE_API		FIoDispatcherFilesystemStats();
	CORE_API		~FIoDispatcherFilesystemStats();
	CORE_API bool	CsvTick(float DeltaTime);
	CORE_API void	OnReadRequestsQueued(uint64 ByteCount, uint64 ReadCount);
	CORE_API void	OnFilesystemReadStarted(uint64 FileHandle, uint64 Offset, uint64 Size);
	CORE_API void 	OnFilesystemReadCompleted(uint64 FileHandle, uint64 Offset, uint64 Size);
	CORE_API void 	OnReadRequestsCompleted(uint64 ByteCount, uint64 ReadCount);
	CORE_API void 	OnDecompressQueued(uint64 CompressedSize, uint64 UncompressedSize);
	CORE_API void 	OnDecompressComplete(uint64 CompressedSize, uint64 UncompressedSize);
	CORE_API void	OnBytesScattered(int64 BytesScattered);
	CORE_API void 	OnBlockCacheStore(uint64 NumBytes);
	CORE_API void 	OnBlockCacheHit(uint64 NumBytes);
	CORE_API void 	OnBlockCacheMiss(uint64 NumBytes);
	CORE_API void 	OnTocMounted(uint64 AllocatedSize);
	CORE_API void 	OnTocUnmounted(uint64 AllocatedSize);
	CORE_API void 	OnBufferReleased();
	CORE_API void 	OnBufferAllocated();

private:
	void			OnSequentialRead();
	void			OnSeek(uint64 PrevOffset, uint64 NewOffset);
	void			OnHandleChangeSeek();

#if COUNTERSTRACE_ENABLED
	FCountersTrace::FCounterAtomicInt	QueuedReadRequestsSizeCounter;
	FCountersTrace::FCounterInt			CompletedReadRequestsSizeCounter;
	FCountersTrace::FCounterInt 		QueuedCompressedSizeCounter;
	FCountersTrace::FCounterInt 		QueuedUncompressedSizeCounter;
	FCountersTrace::FCounterInt 		CompletedCompressedSizeCounter;
	FCountersTrace::FCounterInt 		CompletedUncompressedSizeCounter;
	FCountersTrace::FCounterInt 		FileSystemSeeksTotalDistanceCounter;
	FCountersTrace::FCounterInt 		FileSystemSeeksForwardCountCounter;
	FCountersTrace::FCounterInt 		FileSystemSeeksBackwardCountCounter;
	FCountersTrace::FCounterInt 		FileSystemSeeksChangeHandleCountCounter;
	FCountersTrace::FCounterInt 		FileSystemCompletedRequestsSizeCounter;
	FCountersTrace::FCounterInt 		BlockCacheStoredSizeCounter;
	FCountersTrace::FCounterInt 		BlockCacheHitSizeCounter;
	FCountersTrace::FCounterInt 		BlockCacheMissedSizeCounter;
	FCountersTrace::FCounterInt 		ScatteredSizeCounter;
	FCountersTrace::FCounterInt 		TocMemoryCounter;
	FCountersTrace::TCounter<std::atomic<int64>, TraceCounterType_Int> AvailableBuffersCounter;
#endif
#if CSV_PROFILER_STATS
	std::atomic_uint64_t				QueuedFilesystemReadBytes = 0;
	std::atomic_uint64_t 				QueuedFilesystemReads = 0;
	uint64 								QueuedUncompressBytesIn = 0;
	uint64 								QueuedUncompressBytesOut = 0;
	uint64 								QueuedUncompressBlocks = 0;
#endif
	FTSTicker::FDelegateHandle			TickerHandle;
	FFileReadInfo						LastFileReadInfo;
};

#else
class FIoDispatcherFilesystemStats
{
public:
			FIoDispatcherFilesystemStats() {}
			~FIoDispatcherFilesystemStats() = default;
	void	OnReadRequestsQueued(uint64 ByteCount, uint64 ReadCount) {};
	void	OnFilesystemReadStarted(uint64 FileHandle, uint64 Offset, uint64 Size) {};
	void 	OnFilesystemReadCompleted(uint64 FileHandle, uint64 Offset, uint64 Size) {};
	void 	OnReadRequestsCompleted(uint64 ByteCount, uint64 ReadCount) {};
	void 	OnDecompressQueued(uint64 CompressedSize, uint64 UncompressedSize) {};
	void 	OnDecompressComplete(uint64 CompressedSize, uint64 UncompressedSize) {};
	void	OnBytesScattered(int64 BytesScattered) {};
	void 	OnBlockCacheStore(uint64 NumBytes) {};
	void 	OnBlockCacheHit(uint64 NumBytes) {};
	void 	OnBlockCacheMiss(uint64 NumBytes) {};
	void	OnSequentialRead() {};
	void 	OnSeek(uint64 LastOffset, uint64 NewOffset) {};
	void 	OnHandleChangeSeek() {};
	void 	OnTocMounted(uint64 AllocatedSize) {};
	void 	OnTocUnmounted(uint64 AllocatedSize) {};
	void 	OnBufferReleased() {};
	void 	OnBufferAllocated() {};
};

#endif // UE_IO_DISPATCHER_FILESYSTEM_STATS_ENABLED 

} // namesapce UE
