// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "HAL/Platform.h"
#include "IO/IoDispatcher.h"
#include "IO/IoChunkId.h"
#include "Containers/StringFwd.h"

// This header is used for IoDispatcher functionality we don't want to expose outside of the engine

struct FIoRequestDebugInfo
{
	const TCHAR* BackendName = nullptr;
};

class FIoDispatcherInternal final
{
public:
	static CORE_API bool HasPackageData();
	static CORE_API void MountContainerMeta(const FString& FilePath);
	static CORE_API FUtf8StringView GetFilename(const FIoChunkId& ChunkId, FUtf8StringBuilderBase& OutFilename, FUtf8StringView& OutContainerName);
	static CORE_API void GetDebugInfo(const FIoRequest& IoRequest, FIoRequestDebugInfo& OutDebugInfo);
};
