// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IO/IoDispatcher.h"

class FIoRequestImpl;

class FIoBatchImpl
{
public:
	TFunction<void()> Callback;
	FEvent* Event = nullptr;
	FGraphEventRef GraphEvent;
	TAtomic<uint32> UnfinishedRequestsCount{ 0 };
};

