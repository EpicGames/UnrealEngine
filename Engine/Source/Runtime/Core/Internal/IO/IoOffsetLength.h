// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "HAL/UnrealMemory.h"

/**
 * Combined offset and length.
 */
struct FIoOffsetAndLength
{
	FIoOffsetAndLength() = default;

	explicit FIoOffsetAndLength(const uint64 Offset, const uint64 Length)
	{
		SetOffset(Offset);
		SetLength(Length);
	}

	inline uint64 GetOffset() const
	{
		return OffsetAndLength[4]
			| (uint64(OffsetAndLength[3]) << 8)
			| (uint64(OffsetAndLength[2]) << 16)
			| (uint64(OffsetAndLength[1]) << 24)
			| (uint64(OffsetAndLength[0]) << 32)
			;
	}

	inline uint64 GetLength() const
	{
		return OffsetAndLength[9]
			| (uint64(OffsetAndLength[8]) << 8)
			| (uint64(OffsetAndLength[7]) << 16)
			| (uint64(OffsetAndLength[6]) << 24)
			| (uint64(OffsetAndLength[5]) << 32)
			;
	}

	inline void SetOffset(uint64 Offset)
	{
		OffsetAndLength[0] = uint8(Offset >> 32);
		OffsetAndLength[1] = uint8(Offset >> 24);
		OffsetAndLength[2] = uint8(Offset >> 16);
		OffsetAndLength[3] = uint8(Offset >>  8);
		OffsetAndLength[4] = uint8(Offset >>  0);
	}

	inline void SetLength(uint64 Length)
	{
		OffsetAndLength[5] = uint8(Length >> 32);
		OffsetAndLength[6] = uint8(Length >> 24);
		OffsetAndLength[7] = uint8(Length >> 16);
		OffsetAndLength[8] = uint8(Length >> 8);
		OffsetAndLength[9] = uint8(Length >> 0);
	}

	/** Returns true if either the length or offset have been set, otherwise false */
	inline bool IsValid() const
	{
		return !FMemory::MemIsZero(OffsetAndLength, 10);
	}

private:
	// We use 5 bytes for offset and size, this is enough to represent
	// an offset and size of 1PB
	uint8 OffsetAndLength[5 + 5];
};
