// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Algo/BinarySearch.h"
#include "Algo/IsSorted.h"
#include "Containers/AllowShrinking.h"

template<typename T>
class TIoPriorityQueue
{
private:
	struct TInternalQueue
	{
		TInternalQueue(int32 InPriority)
			: Priority(InPriority)
		{

		}

		// Returns true if outer queue should be removed
		bool Remove(T* Item)
		{
			T* Prev = Item->Prev;
			T* Next = Item->Next;
			Item->Prev = nullptr;
			Item->Next = nullptr;
			if (Prev && Next)
			{
				Prev->Next = Next;
				Next->Prev = Prev;
				return false;
			}
			if (Prev)
			{
				Prev->Next = Next;
			}
			else
			{
				Head = Next;
			}
			if (Next)
			{
				Next->Prev = Prev;
			}
			else
			{
				Tail = Prev;
			}
			if (!Head)
			{
				check(!Tail);
				return true;
			}
			return false;
		}
		TInternalQueue* NextFree = nullptr;
		T* Head = nullptr;
		T* Tail = nullptr;
		int32 Priority;
	};

public:
	class TIterator
	{
	public:
		friend TIoPriorityQueue;

		TIterator& operator++()
		{
			Current = Next;
			Next = Current ? Current->Next : nullptr;
			return *this;
		}

		TIterator operator++(int)
		{
			TIterator Tmp(*this);
			Current = Next;
			Next = Current ? Current->Next : nullptr;
			return Tmp;
		}

		T& operator*() const
		{
			check(Current);
			check(InternalQueue);
			check(Current->Priority == InternalQueue->Priority);
			return *Current;
		}

		T* operator->() const
		{
			check(Current);
			check(InternalQueue);
			check(Current->Priority == InternalQueue->Priority);
			return Current;
		}

		explicit operator bool() const
		{
			return !!Current;
		}

		void RemoveCurrent()
		{
			check(Current);
			check(InternalQueue);
			check(Current->Priority == InternalQueue->Priority);
			if (InternalQueue->Remove(Current))
			{
				Outer.RemoveQueueByPriority(InternalQueue->Priority);
			}
			Current = reinterpret_cast<T*>(-1);
		}

	private:
		TIterator(TIoPriorityQueue<T>& InOuter, int32 InPriority)
			: Outer(InOuter)
		{
			InternalQueue = Outer.FindQueueByPriority(InPriority);
			if (InternalQueue)
			{
				Current = InternalQueue->Head;
				Next = Current->Next;
			}
			else
			{
				Current = Next = nullptr;
			}
		}

		TIoPriorityQueue<T>& Outer;
		TInternalQueue* InternalQueue;
		T* Current;
		T* Next;
	};

	bool IsEmpty() const
	{
		return InternalQueues.IsEmpty();
	}

	int32 GetMaxPriority() const
	{
		if (InternalQueues.IsEmpty())
		{
			return MIN_int32;
		}
		return InternalQueues.Last()->Priority;
	}

	void Push(T* Item, int32 Priority)
	{
		check(!Item->Prev);
		check(!Item->Next);
		Item->Priority = Priority;
		TInternalQueue& Queue = FindOrAddQueueByPriority(Priority);
		if (!Queue.Head)
		{
			check(!Queue.Tail);
			Queue.Head = Queue.Tail = Item;
		}
		else
		{
			Item->Prev = Queue.Tail;
			Queue.Tail->Next = Item;
			Queue.Tail = Item;
		}
	}

	T* Pop()
	{
		if (InternalQueues.IsEmpty())
		{
			return nullptr;
		}
		int QueueIndex = InternalQueues.Num() - 1;
		TInternalQueue* QueueWithHighestPriority = InternalQueues[QueueIndex];
		check(QueueWithHighestPriority);
		check(QueueWithHighestPriority->Head);
		check(QueueWithHighestPriority->Tail);

		T* Item = QueueWithHighestPriority->Head;
		check(Item);
		check(Item->Priority == QueueWithHighestPriority->Priority);
		T* Next = Item->Next;
		if (Next)
		{
			Next->Prev = nullptr;
			QueueWithHighestPriority->Head = Next;
		}
		else
		{
			check(Item == QueueWithHighestPriority->Tail);
			QueueWithHighestPriority->Head = QueueWithHighestPriority->Tail = nullptr;
			RemoveQueueAtIndex(QueueIndex);
		}
		Item->Prev = Item->Next = nullptr;

		return Item;
	}

	TIterator CreateIterator(int32 Priority)
	{
		return TIterator(*this, Priority);
	}

	void Remove(T* Item)
	{
		T* Prev = Item->Prev;
		T* Next = Item->Next;
		if (Prev && Next)
		{
			Item->Prev = nullptr;
			Item->Next = nullptr;
			Prev->Next = Next;
			Next->Prev = Prev;
			return;
		}
		int32 QueueIndex;
		TInternalQueue* InternalQueue = FindQueueByPriority(Item->Priority, QueueIndex);
		check(InternalQueue);
		if (InternalQueue->Remove(Item))
		{
			RemoveQueueAtIndex(QueueIndex);
		}
	}

	void Reprioritize(T* Item, int32 NewPriority)
	{
		Remove(Item);
		Push(Item, NewPriority);
	}

	void MergeInto(TIoPriorityQueue<T>& Other, int32 Priority)
	{
		int32 QueueIndex;
		TInternalQueue* Queue = FindQueueByPriority(Priority, QueueIndex);
		if (!Queue)
		{
			return;
		}
		TInternalQueue& OtherQueue = Other.FindOrAddQueueByPriority(Priority);
		if (OtherQueue.Head)
		{
			check(OtherQueue.Tail);
			Queue->Head->Prev = OtherQueue.Tail;
			OtherQueue.Tail->Next = Queue->Head;
			OtherQueue.Tail = Queue->Tail;
		}
		else
		{
			OtherQueue.Head = Queue->Head;
			OtherQueue.Tail = Queue->Tail;
		}
		Queue->Head = Queue->Tail = nullptr;
		RemoveQueueAtIndex(QueueIndex);
	}

private:
	TInternalQueue* FindQueueByPriority(int32 Priority) const
	{
		int32 Index = Algo::LowerBoundBy(InternalQueues, Priority, &TInternalQueue::Priority);
		if (InternalQueues.IsValidIndex(Index) && InternalQueues[Index]->Priority == Priority)
		{
			return InternalQueues[Index];
		}
		return nullptr;
	}

	TInternalQueue* FindQueueByPriority(int32 Priority, int32& OutIndex) const
	{
		int32 Index = Algo::LowerBoundBy(InternalQueues, Priority, &TInternalQueue::Priority);
		if (InternalQueues.IsValidIndex(Index) && InternalQueues[Index]->Priority == Priority)
		{
			OutIndex = Index;
			return InternalQueues[Index];
		}
		return nullptr;
	}

	TInternalQueue& FindOrAddQueueByPriority(int32 Priority)
	{
		int32 Index = Algo::LowerBoundBy(InternalQueues, Priority, &TInternalQueue::Priority);
		if (!InternalQueues.IsValidIndex(Index) || InternalQueues[Index]->Priority != Priority)
		{
			InternalQueues.Insert(AllocInternalQueue(Priority), Index);
		}
		return *InternalQueues[Index];
	}

	void RemoveQueueByPriority(int32 Priority)
	{
		int32 QueueIndex;
		if (TInternalQueue* InternalQueue = FindQueueByPriority(Priority, QueueIndex))
		{
			RemoveQueueAtIndex(QueueIndex);
		}
	}

	void RemoveQueueAtIndex(int32 Index)
	{
		FreeInternalQueue(InternalQueues[Index]);
		InternalQueues.RemoveAt(Index, EAllowShrinking::No);
	}

	TInternalQueue* AllocInternalQueue(int32 Priority)
	{
		if (TInternalQueue* FromPool = FirstFreeQueue)
		{
			FirstFreeQueue = FromPool->NextFree;
			FromPool->NextFree = nullptr;
			FromPool->Priority = Priority;
			return FromPool;
		}
		return new TInternalQueue(Priority);
	}

	void FreeInternalQueue(TInternalQueue* InternalQueue)
	{
		check(!InternalQueue->Head);
		check(!InternalQueue->Tail);
		InternalQueue->NextFree = FirstFreeQueue;
		FirstFreeQueue = InternalQueue;
	}

	TArray<TInternalQueue*> InternalQueues; // Sorted by priority
	TInternalQueue* FirstFreeQueue = nullptr;
};