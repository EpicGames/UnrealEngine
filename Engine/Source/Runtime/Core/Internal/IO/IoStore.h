// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/MappedFileHandle.h"
#include "Containers/Array.h"
#include "Experimental/ConcurrentLinearAllocator.h"
#include "HAL/Platform.h"
#include "HAL/UnrealMemory.h"
#include "IO/IoContainerId.h"
#include "IO/IoDispatcher.h"
#include "IO/IoOffsetLength.h"
#include "Logging/LogMacros.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/Guid.h"
#include "Misc/SecureHash.h"
#include "Misc/TVariant.h"
#include "UObject/NameTypes.h"

CORE_API DECLARE_LOG_CATEGORY_EXTERN(LogIoStore, Log, All);

/**
 * I/O store container format version
 */
enum class EIoStoreTocVersion : uint8
{
	Invalid = 0,
	Initial,
	DirectoryIndex,
	PartitionSize,
	PerfectHash,
	PerfectHashWithOverflow,
	OnDemandMetaData,
	RemovedOnDemandMetaData,
	ReplaceIoChunkHashWithIoHash,
	LatestPlusOne,
	Latest = LatestPlusOne - 1
};

/**
 * I/O Store TOC header.
 */
struct FIoStoreTocHeader
{
	static constexpr inline char TocMagicImg[] = "-==--==--==--==-";

	uint8	TocMagic[16];
	uint8	Version;
	uint8	Reserved0 = 0;
	uint16	Reserved1 = 0;
	uint32	TocHeaderSize;
	uint32	TocEntryCount;
	uint32	TocCompressedBlockEntryCount;
	uint32	TocCompressedBlockEntrySize;	// For sanity checking
	uint32	CompressionMethodNameCount;
	uint32	CompressionMethodNameLength;
	uint32	CompressionBlockSize;
	uint32	DirectoryIndexSize;
	uint32	PartitionCount = 0;
	FIoContainerId ContainerId;
	FGuid	EncryptionKeyGuid;
	EIoContainerFlags ContainerFlags;
	uint8	Reserved3 = 0;
	uint16	Reserved4 = 0;
	uint32	TocChunkPerfectHashSeedsCount = 0;
	uint64	PartitionSize = 0;
	uint32	TocChunksWithoutPerfectHashCount = 0;
	uint32	Reserved7 = 0;
	uint64	Reserved8[5] = { 0 };

	void MakeMagic()
	{
		FMemory::Memcpy(TocMagic, TocMagicImg, sizeof TocMagic);
	}

	bool CheckMagic() const
	{
		return FMemory::Memcmp(TocMagic, TocMagicImg, sizeof TocMagic) == 0;
	}
};

enum class FIoStoreTocEntryMetaFlags : uint8
{
	None,
	Compressed		= (1 << 0),
	MemoryMapped	= (1 << 1)
};

ENUM_CLASS_FLAGS(FIoStoreTocEntryMetaFlags);

/**
 * TOC entry meta data
 */
struct FIoStoreTocEntryMeta
{
	// Source data hash (i.e. not the on disk data)
	FIoHash ChunkHash;
	FIoStoreTocEntryMetaFlags Flags = FIoStoreTocEntryMetaFlags::None;
	uint8 Pad[3] = { 0 };
};

/**
 * Compression block entry.
 */
struct FIoStoreTocCompressedBlockEntry
{
	static constexpr uint32 OffsetBits = 40;
	static constexpr uint64 OffsetMask = (1ull << OffsetBits) - 1ull;
	static constexpr uint32 SizeBits = 24;
	static constexpr uint32 SizeMask = (1 << SizeBits) - 1;
	static constexpr uint32 SizeShift = 8;

	inline uint64 GetOffset() const
	{
		const uint64* Offset = reinterpret_cast<const uint64*>(Data);
		return *Offset & OffsetMask;
	}

	inline void SetOffset(uint64 InOffset)
	{
		uint64* Offset = reinterpret_cast<uint64*>(Data);
		*Offset = InOffset & OffsetMask;
	}

	inline uint32 GetCompressedSize() const
	{
		const uint32* Size = reinterpret_cast<const uint32*>(Data) + 1;
		return (*Size >> SizeShift) & SizeMask;
	}

	inline void SetCompressedSize(uint32 InSize)
	{
		uint32* Size = reinterpret_cast<uint32*>(Data) + 1;
		*Size |= (uint32(InSize) << SizeShift);
	}

	inline uint32 GetUncompressedSize() const
	{
		const uint32* UncompressedSize = reinterpret_cast<const uint32*>(Data) + 2;
		return *UncompressedSize & SizeMask;
	}

	inline void SetUncompressedSize(uint32 InSize)
	{
		uint32* UncompressedSize = reinterpret_cast<uint32*>(Data) + 2;
		*UncompressedSize = InSize & SizeMask;
	}

	inline uint8 GetCompressionMethodIndex() const
	{
		const uint32* Index = reinterpret_cast<const uint32*>(Data) + 2;
		return static_cast<uint8>(*Index >> SizeBits);
	}

	inline void SetCompressionMethodIndex(uint8 InIndex)
	{
		uint32* Index = reinterpret_cast<uint32*>(Data) + 2;
		*Index |= uint32(InIndex) << SizeBits;
	}

private:
	/* 5 bytes offset, 3 bytes for size / uncompressed size and 1 byte for compresseion method. */
	uint8 Data[5 + 3 + 3 + 1];
};

/**
 * TOC resource read options.
 */
enum class EIoStoreTocReadOptions
{
	Default,
	ReadDirectoryIndex	= (1 << 0),
	ReadTocMeta			= (1 << 1),
	ReadAll				= ReadDirectoryIndex | ReadTocMeta
};
ENUM_CLASS_FLAGS(EIoStoreTocReadOptions);


struct FIoStoreTocResourceStorage
{
	FIoStoreTocResourceStorage() = default;
	CORE_API explicit FIoStoreTocResourceStorage(const TCHAR* TocFilePath);

	bool IsLoaded() const { return !Data.IsType<FEmptyVariantState>(); }
	CORE_API uint64 GetAllocatedSize() const;

	template<typename T>
	TConstArrayView<T> ChopView(int32 Num)
	{
		const int32 RequestedSize = Num * static_cast<int32>(sizeof(T));
		TConstArrayView<uint8> Bytes = ChopBytes(RequestedSize);
		check(RequestedSize == Bytes.Num());
		return TConstArrayView<T>(reinterpret_cast<const T*>(Bytes.GetData()), Num);
	}

	template<typename T>
	TArray<T> ChopArray(int32 Num)
	{
		TArray<T> Result(ChopView<T>(Num));
		if (!Result.IsEmpty())
		{
			ReleaseOwnershipOfLastBlock();
		}
		return Result;
	}

	CORE_API void FinalizeRead();

private:
	CORE_API void ReleaseOwnershipOfLastBlock();

	struct FIOStoreAllocationTag
	{
		static constexpr uint32 BlockSize = 64 * 1024;
		static constexpr bool AllowOversizedBlocks = true;
		static constexpr bool RequiresAccurateSize = true;
		static constexpr bool InlineBlockAllocation = false;
		static constexpr const char* TagName = "IOStoreLinear";

		using Allocator = TBlockAllocationLockFreeCache<BlockSize, FAlignedAllocator>;
	};
	using FDataAllocator = TConcurrentLinearArrayAllocator<FIOStoreAllocationTag>;
	using FLocalAllocator = TInlineAllocator<10>;
	struct FMappedFile
	{
		TUniquePtr<IMappedFileHandle> MappedFile;
		TArray<TUniquePtr<IMappedFileRegion>, FLocalAllocator> MappedRegions;
		TArray<TArray<uint8, FDataAllocator>, FLocalAllocator> OwnedRegions;
		int32 Cursor = 0;
		bool bLastReadBlockWasOwned = false;
	};
	struct FReadBlocks
	{
		TUniquePtr<IFileHandle> File;
		TArray<TArray<uint8, FDataAllocator>, FLocalAllocator> Blocks;
	};

	static CORE_API TConstArrayView<uint8> ChopBytes(FMappedFile& File, int32 Size);
	static CORE_API TConstArrayView<uint8> ChopBytes(FReadBlocks& File, int32 Size);

	TConstArrayView<uint8> ChopBytes(FEmptyVariantState&, int32 Size)
	{
		return {};
	}

	TConstArrayView<uint8> ChopBytes(int32 Size)
	{
		return Visit([this, Size](auto&& Value) { return ChopBytes(Value, Size); }, Data);
	}

	TVariant<FEmptyVariantState, FMappedFile, FReadBlocks> Data;
};

struct FIoStoreTocResourceView
{
	FIoStoreTocHeader Header;
	TConstArrayView<FIoChunkId> ChunkIds;
	TConstArrayView<FIoOffsetAndLength> ChunkOffsetLengths;
	TConstArrayView<int32> ChunkPerfectHashSeeds;
	TConstArrayView<int32> ChunkIndicesWithoutPerfectHash;
	TConstArrayView<FIoStoreTocCompressedBlockEntry> CompressionBlocks;

	TArray<FName> CompressionMethods;
	FSHAHash SignatureHash;
	TConstArrayView<FSHAHash> ChunkBlockSignatures;
	TConstArrayView<uint8> DirectoryIndexBuffer;
	TConstArrayView<FIoStoreTocEntryMeta> ChunkMetas;
	TArray<FIoStoreTocEntryMeta> LegacyChunkMetas; // For Version < EIoStoreTocVersion::ReplaceIoChunkHashWithIoHash otherwise empty

	[[nodiscard]] CORE_API static FIoStatus Read(const TCHAR* TocFilePath, EIoStoreTocReadOptions ReadOptions, FIoStoreTocResourceView& OutTocResource, FIoStoreTocResourceStorage& OutTocResourceStorage);
	[[nodiscard]] CORE_API FIoStoreTocChunkInfo GetTocChunkInfo(int32 TocEntryIndex) const;
};

/**
 * Container TOC data.
 */
struct FIoStoreTocResource
{
	enum { CompressionMethodNameLen = 32 };

	FIoStoreTocHeader Header;

	TArray<FIoChunkId> ChunkIds;

	TArray<FIoOffsetAndLength> ChunkOffsetLengths;

	TArray<int32> ChunkPerfectHashSeeds;

	TArray<int32> ChunkIndicesWithoutPerfectHash;

	TArray<FIoStoreTocCompressedBlockEntry> CompressionBlocks;

	TArray<FName> CompressionMethods;

	FSHAHash SignatureHash;

	TArray<FSHAHash> ChunkBlockSignatures;

	TArray<uint8> DirectoryIndexBuffer; 

	TArray<FIoStoreTocEntryMeta> ChunkMetas;

	[[nodiscard]] CORE_API FIoStoreTocChunkInfo GetTocChunkInfo(int32 TocEntryIndex) const;

	[[nodiscard]] CORE_API static FIoStoreTocResource BuildResourceFromMappedView(const FIoStoreTocResourceView& View);
	[[nodiscard]] CORE_API static FIoStatus Read(const TCHAR* TocFilePath, EIoStoreTocReadOptions ReadOptions, FIoStoreTocResource& OutTocResource);
	[[nodiscard]] CORE_API static TIoStatusOr<uint64> Write(const TCHAR* TocFilePath, FIoStoreTocResource& TocResource, uint32 CompressionBlockSize, uint64 MaxPartitionSize, const FIoContainerSettings& ContainerSettings);

	[[nodiscard]] CORE_API static uint64 HashChunkIdWithSeed(int32 Seed, const FIoChunkId& ChunkId);

	[[nodiscard]] CORE_API FIoStoreTocResourceView AsView() const;
};

/**
 * This is the next level up from FIoStoreTocResource - it loads the toc but also resolves the directory index so you get 
 * filenames. If you don't need that, use FIoStoreTocResource.
*/
class IIoStoreTocReader
{
public:
	virtual const FIoStoreTocResourceView& GetTocResource() const = 0;
	[[nodiscard]] virtual FIoStoreTocChunkInfo GetTocChunkInfo(int32 TocEntryIndex) const = 0;
	CORE_API virtual ~IIoStoreTocReader();

	// Creates an instance for accessing the TOC. The TocFilePath should be a path to the
	// .utoc file (not the ucas or undecorated). 
	CORE_API static TIoStatusOr<TUniquePtr<IIoStoreTocReader>> ReadFromDisk(const TCHAR* TocFilePath, const TMap<FGuid, FAES::FAESKey>& DecryptionKeys);
};