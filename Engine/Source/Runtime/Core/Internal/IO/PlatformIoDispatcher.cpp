// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlatformIoDispatcher.h"

#if PLATFORM_IMPLEMENTS_IO
#include "HAL/PreprocessorHelpers.h"
#include COMPILED_PLATFORM_HEADER_WITH_PREFIX(IO, PlatformIoDispatcher.h)
#else
#include "IO/GenericPlatformIoDispatcher.h"
#endif

DEFINE_LOG_CATEGORY(LogPlatformIoDispatcher);

namespace UE
{

TUniquePtr<IPlatformIoDispatcher> GPlatformIoDispatcher;

extern TUniquePtr<IPlatformIoDispatcher> MakeGenericPlatformIoDispatcher(FPlatformIoDispatcherCreateParams&&);

void FPlatformIoDispatcher::Create(FPlatformIoDispatcherCreateParams&& Params)
{
	if (Params.bForceGeneric)
	{
		GPlatformIoDispatcher = MakeGenericPlatformIoDispatcher(MoveTemp(Params));
	}
	else
	{
		GPlatformIoDispatcher = FPlatformIoDispatcherFactory::Create(MoveTemp(Params));
	}
}

void FPlatformIoDispatcher::Initialize()
{
	if (GPlatformIoDispatcher.IsValid())
	{
		GPlatformIoDispatcher->Initialize();
	}
}

void FPlatformIoDispatcher::Shutdown()
{
	GPlatformIoDispatcher.Reset();
}

IPlatformIoDispatcher& FPlatformIoDispatcher::Get()
{
	return *GPlatformIoDispatcher;
}

IPlatformIoDispatcher* FPlatformIoDispatcher::TryGet()
{
	return GPlatformIoDispatcher.Get();
}

} // namespace UE
