// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IO/IoDispatcherPriority.h"
#include "IO/IoStatus.h"
#include "IO/IoBuffer.h"
#include "Logging/LogMacros.h"
#include "Math/NumericLimits.h"
#include "Memory/MemoryFwd.h"
#include "Misc/EnumClassFlags.h"
#include "Templates/Function.h"
#include "UObject/NameTypes.h"

#if !defined(PLATFORM_IMPLEMENTS_IO)
#define PLATFORM_IMPLEMENTS_IO 0
#endif

CORE_API DECLARE_LOG_CATEGORY_EXTERN(LogPlatformIoDispatcher, Log, All);

namespace UE
{

/** Handle to a file on disk. */
struct FIoFileHandle
{
			FIoFileHandle() = default;
			explicit FIoFileHandle(uint64 InHandle)
				: Handle(InHandle)
			{ }

	uint64	Value() const { return Handle; }
	bool	IsValid() const { return Handle != 0; }

	friend uint32 GetTypeHash(FIoFileHandle FileHandle) { return GetTypeHash(FileHandle.Handle); }
	friend	bool operator==(FIoFileHandle LHS, FIoFileHandle RHS) { return LHS.Handle == RHS.Handle; }
	friend	bool operator!=(FIoFileHandle LHS, FIoFileHandle RHS) { return LHS.Handle != RHS.Handle; }
	friend	bool operator<(FIoFileHandle LHS, FIoFileHandle RHS) { return LHS.Handle < RHS.Handle; }

private:
	uint64 Handle = 0;
};

/** File stats returned when opening a file for reading. */
struct FIoFileStat
{
	uint64 FileSize = 0;
};

/** Specifies whether a file contains encrypted or signed I/O store chunks. */
enum class EIoFilePropertyFlags : uint8
{
	None		= 0,
	Encrypted	= (1 << 0),
	Signed		= (1 << 1)
};
ENUM_CLASS_FLAGS(EIoFilePropertyFlags);

/**
 * File properties are used to determine whether it's possible to use
 * direct-read, i.e. reading directly from the file to destination buffer and use
 * hardware decompression.
 */
struct FIoFileProperties
{
	TConstArrayView<FName>	CompressionMethods;
	uint32					CompressionBlockSize = 0; 
	EIoFilePropertyFlags	Flags = EIoFilePropertyFlags::None;
};

/**
 * Handle to a platform file read request. Owned by the caller and needs to be deleted when completed.
 */
struct FIoFileReadRequest
{
			FIoFileReadRequest() = default;
			explicit FIoFileReadRequest(uint64 InHandle)
				: Handle(InHandle)
			{ }

	uint64	Value() const { return Handle; }
	bool	IsValid() const { return Handle != 0; }

private:
	uint64 Handle = 0;
};

/**
 * The scatter-gather parameters specifies the I/O store encoded block boundaries, 
 * whether the block is compressed and or encrypted, whether to check the block
 * signature hash and from where in the uncompressed block to copy the block data
 * to the destination buffer.
 */
class FIoScatterGatherRequestParams
{
public:
	struct FScatterParams
	{
		uint64		BlockFileOffset			= 0;
		uint32		BlockCompressedSize		= 0;
		uint32		BlockUncompresedSize	= 0;
		uint64		ScatterOffset			= 0;
		uint64		ScatterSize				= 0;
		uint64		DestinationOffset		= 0;
		uint32		BlockIndex				= MAX_uint32;
		FName		CompressionMethod		= NAME_None;
		FMemoryView	EncryptionKey;
		FMemoryView	BlockHash;
	};

	using FScatterParamsArray = TArray<FScatterParams, TInlineAllocator<4>>;

	/**
	 * Creates a new set of scatter-gather parameters.
	 *
	 * @param	InFileHandle		Handle to the file to read from.
	 * @param	InDestination		Reference to the destination buffer.
	 * @param	InDestinationSize	Size in bytes of the destination buffer.
	 * @param	InUserData			Optional user data associated with the request.
	 * @param	InPriority			The priority of the request.
	 */
	FIoScatterGatherRequestParams(
		FIoFileHandle InFileHandle,
		FIoBuffer& InDestination,
		uint64 InDestinationSize,
		void* InUserData,
		int32 InPriority)
			: FileHandle(InFileHandle)
			, Destination(InDestination)
			, DestinationSize(InDestinationSize)
			, UserData(InUserData)
			, Priority(InPriority)
	{ }

	/**
	 *	Add block scatter parameters.
	 *
	 *	@param	BlockFileOffset			Offset of the encoded block in the file.
	 *	@param	BlockIndex				Index of the block in the TOC.
	 *	@param	BlockCompressedSize		Compressed size of the block.
	 *	@param	BlockUncompressedSize	Uncompressed size of the block.
	 *	@param	ScatterOffset			Offset in the uncompressed block from where to copy the data to the destination buffer.
	 *	@param	ScatterSize				Number of bytes to copy from the uncompressed block.
	 *	@param	DestinationOffset		Offset in the destination buffer to copy the data from the uncompressed buffer.
	 *	@param	CompressionMethod		Method used to compress the block data.
	 *	@param	EncryptionKey			Optional memory view to the encryption key.
	 *	@param	BlockHash				Optional memory view to the block hash for signature check.
	 */
	void Scatter(
		uint64 BlockFileOffset,
		uint32 BlockIndex,
		uint32 BlockCompressedSize,
		uint32 BlockUncompresedSize,
		uint64 ScatterOffset,
		uint64 ScatterSize,
		uint64 DestinationOffset,
		FName CompressionMethod,
		FMemoryView EncryptionKey,
		FMemoryView BlockHash)
	{
		Params.Add(FScatterParams
		{
			.BlockFileOffset		= BlockFileOffset,
			.BlockCompressedSize	= BlockCompressedSize,
			.BlockUncompresedSize	= BlockUncompresedSize,
			.ScatterOffset			= ScatterOffset,
			.ScatterSize			= ScatterSize,
			.DestinationOffset		= DestinationOffset,
			.BlockIndex				= BlockIndex,
			.CompressionMethod		= CompressionMethod,
			.EncryptionKey			= EncryptionKey,
			.BlockHash				= BlockHash
		});
	}

	FIoFileHandle			FileHandle;
	FIoBuffer&				Destination;
	uint64					DestinationSize;
	void*					UserData;
	int32					Priority;
	FScatterParamsArray		Params;
};

/** File read result. */
struct FIoFileReadResult
{
	void*			UserData		= nullptr;
	uint32			FailedBlockId	= MAX_uint32;
	EIoErrorCode	ErrorCode		= EIoErrorCode::Unknown;
};

using FIoFileReadRequestCompleted = TUniqueFunction<void(FIoFileReadResult&&)>;

/** Parameters for reading directly from file on disk to the destination buffer. */
struct FIoDirectReadRequestParams
{
	FIoFileHandle	FileHandle;
	FIoBuffer&		Dst;
	uint64			Offset		= MAX_uint64;
	uint64			Size		= 0;
	void*			UserData	= nullptr;
};

/** Parameters passed to the platform specific I/O dispatcher. */
struct FPlatformIoDispatcherCreateParams
{
	bool bMultithreaded	= true;
	bool bForceGeneric	= false;
};

/** Interface for platform specific I/O dispatcher. */
class IPlatformIoDispatcher
{
public:
	virtual										~IPlatformIoDispatcher() = default;
	/** Initialize the dispatcher. Called after the config system has been initialized. */
	virtual FIoStatus							Initialize() = 0;
	/** Open file for reading. The file properties specifies whether the file is compressed, encrypted and signed. */ 
	virtual TIoStatusOr<FIoFileHandle>			OpenFile(const TCHAR* Filename, const FIoFileProperties& FileProperties, FIoFileStat* OutStat = nullptr) = 0;
	/** Close the file. Assumes no pending I/O requests are inflight for the specified file. */
	virtual FIoStatus							CloseFile(FIoFileHandle FileHandle) = 0;
	/** Issue a scatter-gather request. */
	[[nodiscard]] virtual FIoFileReadRequest	ScatterGather(FIoScatterGatherRequestParams&& Params, FIoFileReadRequestCompleted&& OnCompleted) = 0;
	/** Issue a direct read request. Returns an invalid file handle if direct read is not supported for the specifid file. */ 
	[[nodiscard]] virtual FIoFileReadRequest	ReadDirect(FIoDirectReadRequestParams&& Params, FIoFileReadRequestCompleted&& OnCompleted) = 0;
	/** Tick the dispatcher. Only used when multithreading is disabled. */ 
	virtual bool								Tick() = 0;
	/** Update priority for an inflight I/O request. */ 
	virtual void								UpdatePriority(FIoFileReadRequest Request, int32 NewPriority) = 0;
	/** Cancel inflight I/O request. */ 
	virtual void								CancelRequest(FIoFileReadRequest Request) = 0;
	/** Cancel all inflight I/O request for the specified file. */ 
	virtual void								CancelAllRequests(FIoFileHandle FileHandle) = 0;
	/** Delete the request. Must be called after the completion callback has been triggered. */ 
	virtual void								DeleteRequest(FIoFileReadRequest Request) = 0;
};

class FPlatformIoDispatcher
{
public:
	CORE_API static void					Create(FPlatformIoDispatcherCreateParams&& Config);
	CORE_API static void					Initialize();
	CORE_API static void					Shutdown();
	CORE_API static IPlatformIoDispatcher&	Get();
	CORE_API static IPlatformIoDispatcher*	TryGet();
};

} // namespace UE
