// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlatformIoDispatcherBase.h"

#include "Algo/BinarySearch.h"
#include "Algo/IsSorted.h"
#include "Async/UniqueLock.h"
#include "HAL/Event.h"
#include "HAL/LowLevelMemTracker.h"
#include "IO/IoChunkEncoding.h"
#include "IO/IoStatus.h"
#include "IO/IoDispatcherConfig.h"
#include "IO/PlatformIoDispatcher.h"
#include "Math/NumericLimits.h"
#include "Misc/AES.h"

#include <atomic>

namespace UE
{

////////////////////////////////////////////////////////////////////////////////
template<typename... Args>
bool IsIoErrorAnyOf(EIoErrorCode ErrorCode, Args... Expected)
{
	return ((ErrorCode == Expected) || ...);
}

////////////////////////////////////////////////////////////////////////////////
uint32 FIoFileBlockRequest::NextSeqNo = 0;

////////////////////////////////////////////////////////////////////////////////
FIoQueue::FIoQueue(FPlatformIoDispatcherStats& InStats)
	: Stats(InStats)
{
}

void FIoQueue::Enqueue(FIoFileBlockRequestList&& FileBlockRequests)
{
	while (FIoFileBlockRequest* Request = FileBlockRequests.PopHead())
	{
		check(Request->QueueStatus == FIoFileBlockRequest::EQueueStatus::None);
		Request->QueueStatus = FIoFileBlockRequest::EQueueStatus::Enqueued;
		if (bSortByOffset)
		{
			AddToPrioQueue(*Request);
		}
		else
		{
			Heap.HeapPush(Request, SeqNoLess);
		}

		Stats.OnFileBlockRequestEnqueued(*Request);
	}
}

FIoFileBlockRequest* FIoQueue::Dequeue()
{
	const bool bDequeue = true;
	FIoFileBlockRequest* Request = GetCancelled(bDequeue);
	if (Request == nullptr)
	{
		if (bSortByOffset)
		{
			Request = GetByOffset(bDequeue);
		}
		else if (Heap.IsEmpty() == false)
		{
			Heap.HeapPop(Request, SeqNoLess, EAllowShrinking::No);
		}
	}

	if (Request != nullptr)
	{
		Request->QueueStatus = FIoFileBlockRequest::EQueueStatus::Dequeued;
		Stats.OnFileBlockRequestDequeued(*Request);
	}

	return Request;
}

FIoFileBlockRequest* FIoQueue::Peek()
{
	const bool bDequeue = false;
	FIoFileBlockRequest* Request = GetCancelled(bDequeue);
	if (Request == nullptr)
	{
		if (bSortByOffset)
		{
			Request = GetByOffset(bDequeue);
		}
		else if (Heap.IsEmpty() == false)
		{
			Request = Heap.HeapTop();
		}
	}

	return Request;
}

void FIoQueue::Reprioritize()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(RequestQueueUpdateOrder);

	if (bSortByOffset)
	{
		TArray<FIoFileBlockRequest*> RequestsToReprioritize;
		for (FPrioQueue& PrioQueue : PrioQueues)
		{
			PrioQueue.PeekIndex = INDEX_NONE;

			for (int32 RequestIdx = PrioQueue.ByOffset.Num() - 1; RequestIdx >= 0; --RequestIdx)
			{
				FIoFileBlockRequest* Request = PrioQueue.ByOffset[RequestIdx];
				if (Request->Priority != PrioQueue.Priority)
				{
					RequestsToReprioritize.Add(Request);
					PrioQueue.ByOffset.RemoveAt(RequestIdx, EAllowShrinking::No);
					PrioQueue.BySequence.Remove(Request);
				}
			}
		}

		PrioQueues.RemoveAll([](FPrioQueue& PrioQueue) { return PrioQueue.ByOffset.IsEmpty(); });

		for (FIoFileBlockRequest* Request : RequestsToReprioritize)
		{
			AddToPrioQueue(*Request);
		}
	}
	else if (Heap.IsEmpty() == false)
	{
		Heap.Heapify(SeqNoLess);
	}
}

void FIoQueue::AddToPrioQueue(FIoFileBlockRequest& Request)
{
	const int32 QueueIndex = Algo::LowerBoundBy(PrioQueues, Request.Priority, QueueToPriority, TLess<int32>());
	if (!PrioQueues.IsValidIndex(QueueIndex) || PrioQueues[QueueIndex].Priority != Request.Priority)
	{
		PrioQueues.Insert(FPrioQueue(Request.Priority), QueueIndex);
	}

	check(PrioQueues.IsValidIndex(QueueIndex));
	check(Algo::IsSortedBy(PrioQueues, QueueToPriority, TLess<int32>()));
	FPrioQueue& PrioQueue	= PrioQueues[QueueIndex];
	const int32 Idx			= Algo::UpperBoundBy(PrioQueue.ByOffset, ToFileOffset(&Request), ToFileOffset, FileOffsetLess);

	check(PrioQueue.Priority == Request.Priority);
	PrioQueue.ByOffset.Insert(&Request, Idx);
	PrioQueue.BySequence.AddTail(&Request);
	PrioQueue.PeekIndex = INDEX_NONE;
}

FIoFileBlockRequest* FIoQueue::GetByOffset(bool bDequeue)
{
	FIoFileBlockRequest* Request = nullptr;
	if (!PrioQueues.IsEmpty()) 
	{
		FPrioQueue& PrioQueue	= PrioQueues.Last();
		Request					= GetByOffset(PrioQueue, bDequeue);

		if (PrioQueue.ByOffset.IsEmpty())
		{
			PrioQueues.Pop();
		}
	}

	return Request;
}

FIoFileBlockRequest* FIoQueue::GetByOffset(FPrioQueue& PrioQueue, bool bDequeue)
{
	check(!PrioQueue.ByOffset.IsEmpty() && !PrioQueue.BySequence.IsEmpty());

	int32 RequestIndex = INDEX_NONE;
	if (PrioQueue.PeekIndex != INDEX_NONE)
	{
		RequestIndex = PrioQueue.PeekIndex;
	}
	else 
	{
		bool bHeadRequestTooOld = false;
		if (GIoDispatcherRequestLatencyCircuitBreakerMS > 0)
		{
			// If our oldest request has been unserviced for too long, grab that instead of the next sequential read
			const uint64 ThresholdCycles = FPlatformTime::SecondsToCycles64(GIoDispatcherRequestLatencyCircuitBreakerMS * 1000.0);
			bHeadRequestTooOld = (FPlatformTime::Cycles64() - PrioQueue.BySequence.PeekHead()->TimeCreated) >= ThresholdCycles;
		}

		const bool bChooseByOffset = 
				LastFileOffset.FileHandle.IsValid()
			&&	!bHeadRequestTooOld 
			&&  (GIoDispatcherMaintainSortingOnPriorityChange || LastFileOffset.Priority == PrioQueue.Priority);
		if (bChooseByOffset)
		{
			RequestIndex = Algo::LowerBoundBy(PrioQueue.ByOffset, LastFileOffset, ToFileOffset, FileOffsetLess);
			if (PrioQueue.ByOffset.IsValidIndex(RequestIndex)) 
			{
				if (PrioQueue.ByOffset[RequestIndex]->FileHandle.Value() != LastFileOffset.FileHandle.Value())
				{
					// Changing file handle so switch back to the oldest outstanding request 
					RequestIndex = INDEX_NONE;
				}
			}
		}

		if (PrioQueue.ByOffset.IsValidIndex(RequestIndex) == false)
		{
			RequestIndex = PrioQueue.ByOffset.Find(PrioQueue.BySequence.PeekHead());
			check(PrioQueue.ByOffset[RequestIndex] == PrioQueue.BySequence.PeekHead());
		}
	}

	check(PrioQueue.ByOffset.IsValidIndex(RequestIndex));
	FIoFileBlockRequest* Request = PrioQueue.ByOffset[RequestIndex];
	if (bDequeue)
	{
		PrioQueue.ByOffset.RemoveAt(RequestIndex, EAllowShrinking::No);
		PrioQueue.BySequence.Remove(Request);
		PrioQueue.PeekIndex = INDEX_NONE;
		LastFileOffset		= ToFileOffset(Request);
	}
	else
	{
		PrioQueue.PeekIndex = RequestIndex;
	}

	return Request;
}

FIoFileBlockRequest* FIoQueue::GetCancelled(bool bDequeue)
{
	if (bReprioritizeCancelled)
	{
		bReprioritizeCancelled = false;
		TRACE_CPUPROFILER_EVENT_SCOPE(RequestQueueRemoveCancelled);
		if (bSortByOffset)
		{
			for (FPrioQueue& PrioQueue : PrioQueues)
			{
				PrioQueue.PeekIndex = INDEX_NONE;
				for (int32 RequestIdx = PrioQueue.ByOffset.Num() - 1; RequestIdx >= 0; --RequestIdx)
				{
					FIoFileBlockRequest* Request = PrioQueue.ByOffset[RequestIdx];
					if (Request->ErrorCode == EIoErrorCode::Cancelled)
					{
						CancelledHeap.HeapPush(Request, SeqNoLess);
						PrioQueue.ByOffset.RemoveAt(RequestIdx, EAllowShrinking::No);
						PrioQueue.BySequence.Remove(Request);
					}
				}
			}

			PrioQueues.RemoveAll([](FPrioQueue& PrioQueue) { return PrioQueue.ByOffset.IsEmpty(); });
		}
		else
		{
			for (int32 Idx = Heap.Num() - 1; Idx >= 0; --Idx)
			{
				FIoFileBlockRequest* Request = Heap[Idx];
				if (Request->ErrorCode == EIoErrorCode::Cancelled)
				{
					CancelledHeap.HeapPush(Request, SeqNoLess);
					Heap.RemoveAt(Idx, EAllowShrinking::No);
				}
			}
			Heap.Heapify(SeqNoLess);
		}
	}

	FIoFileBlockRequest* Cancelled = nullptr;
	if (CancelledHeap.IsEmpty() == false)
	{
		if (bDequeue)
		{
			CancelledHeap.HeapPop(Cancelled, SeqNoLess, EAllowShrinking::No);
		}
		else
		{
			Cancelled = CancelledHeap.HeapTop();
		}
	}

	return Cancelled;
}

////////////////////////////////////////////////////////////////////////////////
FIoFileBlockMemoryPool::FIoFileBlockMemoryPool(FPlatformIoDispatcherStats& InStats)
	: Stats(InStats)
{
}

void FIoFileBlockMemoryPool::Initialize(uint32 FileBlockSize, uint32 FilePoolSize, uint32 Alignment)
{
	check(FileBlockSize > 0 && FilePoolSize > 0);

	const uint32 Padding		= Alignment - sizeof(FMemoryBlock);
	const uint32 BlockCount		= FilePoolSize / FileBlockSize;
	const uint32 MemorySize		= FileBlockSize * BlockCount;

	BlockSize = FileBlockSize;
	BlockMemory.Reset(FMemory::Malloc(MemorySize, Alignment));
	MemoryBlocks.SetNum(BlockCount);

	uint8* Memory = reinterpret_cast<uint8*>(BlockMemory.Get());
	for (int32 Idx = 0; Idx < MemoryBlocks.Num(); ++Idx)
	{
		FMemoryBlock& Block = MemoryBlocks[Idx];
		Block.Next		= FreeBlock;
		Block.Memory	= Memory;
		Block.Index		= Idx;

		FreeBlock		= &Block;
		Memory			+= FileBlockSize;

		Stats.OnFileBlockMemoryFreed(BlockSize);
	}
}

void* FIoFileBlockMemoryPool::Alloc(FIoBufferHandle& OutHandle)
{
	TUniqueLock Lock(Mutex);

	if (FMemoryBlock* Block = FreeBlock)
	{
		check(Block->RefCount == 0);

		FreeBlock		= Block->Next;
		Block->Next		= nullptr;
		Block->RefCount = 1;
		OutHandle	= FIoBufferHandle(Block->Index);

		Stats.OnFileBlockMemoryAllocated(BlockSize);
		return Block->Memory;
	}

	OutHandle = FIoBufferHandle();
	return nullptr;
}

bool FIoFileBlockMemoryPool::Free(FIoBufferHandle& Handle)
{
	bool bFreed = false;
	TUniqueLock Lock(Mutex);

	if (Handle.IsValid() == false)
	{
		return bFreed;
	}

	FMemoryBlock& Block = MemoryBlocks[Handle.Value()];
	check(Block.Next == nullptr);
	check(Block.RefCount > 0);

	if (--Block.RefCount == 0)
	{
		Block.Next	= FreeBlock;
		FreeBlock	= &Block;
		bFreed		= true;

		Stats.OnFileBlockMemoryFreed(BlockSize);
	}

	Handle = FIoBufferHandle();
	return bFreed;
}

void FIoFileBlockMemoryPool::AddRef(FIoBufferHandle Handle)
{
	TUniqueLock Lock(Mutex);
	check(Handle.IsValid());
	FMemoryBlock& Block = MemoryBlocks[Handle.Value()];
	Block.RefCount++;
}

////////////////////////////////////////////////////////////////////////////////
FIoChunkBlockMemoryPool::FIoChunkBlockMemoryPool()
{
}

void FIoChunkBlockMemoryPool::Initialize(uint32 MaxBlockCount, uint32 DefaultBlockSize)
{
	MemoryBlocks.SetNum(MaxBlockCount);
	for (uint32 Idx = 0; Idx < MaxBlockCount; ++Idx)
	{
		FMemoryBlock& Block = MemoryBlocks[Idx];
		Block.Next			= FreeBlock;
		Block.Memory		= FMemory::Malloc(DefaultBlockSize);
		Block.Size			= DefaultBlockSize;
		Block.Index			= Idx;
		FreeBlock			= &Block;
	}
}

void* FIoChunkBlockMemoryPool::Alloc(uint32 BlockSize, FIoBufferHandle& OutHandle)
{
	if (FMemoryBlock* Block = FreeBlock)
	{
		FreeBlock	= Block->Next;
		Block->Next	= nullptr;

		if (Block->Size < BlockSize)
		{
			Block->Memory	= FMemory::Realloc(Block->Memory, BlockSize);
			Block->Size		= BlockSize;
		}

		OutHandle = FIoBufferHandle(Block->Index);
		return Block->Memory;
	}

	OutHandle = FIoBufferHandle();
	return nullptr;
}

void* FIoChunkBlockMemoryPool::Realloc(FIoBufferHandle Handle, uint32 BlockSize)
{
	check(Handle.Value() < MemoryBlocks.Num());
	FMemoryBlock& Block = MemoryBlocks[Handle.Value()];

	if (Block.Size < BlockSize)
	{
		Block.Memory	= FMemory::Realloc(Block.Memory, BlockSize);
		Block.Size		= BlockSize;
	}

	return Block.Memory;
}

void FIoChunkBlockMemoryPool::Free(FIoBufferHandle& Handle)
{
	if (Handle.IsValid() == false)
	{
		return;
	}

	check(Handle.Value() < MemoryBlocks.Num());
	FMemoryBlock& Block = MemoryBlocks[Handle.Value()];
	Block.Next			= FreeBlock;
	FreeBlock			= &Block;

	Handle = FIoBufferHandle();
}

////////////////////////////////////////////////////////////////////////////////
FIoFileBlockCache::FIoFileBlockCache(FPlatformIoDispatcherStats& InStats)
	: Stats(InStats)
{
}

FIoFileBlockCache::~FIoFileBlockCache()
{
}

void FIoFileBlockCache::Initialize(uint64 InCacheBlockSize, uint64 InCacheSize)
{
	CacheBlockSize					= InCacheBlockSize;
	const uint64 CacheBlockCount	= InCacheSize / InCacheBlockSize;
	if (CacheBlockCount > 0)
	{
		const uint64 TotalCacheSize	= CacheBlockCount * InCacheBlockSize;
		CacheMemory					= MakeUnique<uint8[]>(TotalCacheSize);

		FCachedBlock* Prev	= &CacheLruHead;
		for (uint64 CacheBlockIndex = 0; CacheBlockIndex < CacheBlockCount; ++CacheBlockIndex)
		{
			FCachedBlock* CachedBlock	= new FCachedBlock();
			CachedBlock->Key			= uint64(-1);
			CachedBlock->Buffer			= CacheMemory.Get() + CacheBlockIndex * InCacheBlockSize;
			Prev->LruNext				= CachedBlock;
			CachedBlock->LruPrev		= Prev;
			Prev						= CachedBlock;
		}
		Prev->LruNext			= &CacheLruTail;
		CacheLruTail.LruPrev	= Prev;
	}
}

bool FIoFileBlockCache::Get(FIoFileBlockRequest& FileBlockRequest)
{
	if (CacheMemory == nullptr)
	{
		return false;
	}

	check(FileBlockRequest.BlockKey.IsValid());
	check(FileBlockRequest.BufferHandle.IsValid());

	FCachedBlock* CachedBlock = CachedBlocks.FindRef(FileBlockRequest.BlockKey.GetHash());
	if (CachedBlock == nullptr)
	{
		Stats.OnFileBlockCacheMiss(CacheBlockSize);
		return false;
	}
	
	CachedBlock->LruPrev->LruNext = CachedBlock->LruNext;
	CachedBlock->LruNext->LruPrev = CachedBlock->LruPrev;

	CachedBlock->LruPrev = &CacheLruHead;
	CachedBlock->LruNext = CacheLruHead.LruNext;

	CachedBlock->LruPrev->LruNext = CachedBlock;
	CachedBlock->LruNext->LruPrev = CachedBlock;

	check(CachedBlock->Buffer);
	Stats.OnFileBlockCacheHit(CacheBlockSize);
	FMemory::Memcpy(FileBlockRequest.Buffer, CachedBlock->Buffer, CacheBlockSize);

	return true;
}

void FIoFileBlockCache::Put(FIoFileBlockRequest& FileBlockRequest)
{
	bool bIsCacheableBlock = CacheMemory != nullptr && FileBlockRequest.BytesUsed < FileBlockRequest.Size;
	if (!bIsCacheableBlock)
	{
		return;
	}
	check(FileBlockRequest.BufferHandle.IsValid());
	FCachedBlock* BlockToReplace = CacheLruTail.LruPrev;
	if (BlockToReplace == &CacheLruHead)
	{
		return;
	}
	check(BlockToReplace);
	CachedBlocks.Remove(BlockToReplace->Key);
	BlockToReplace->Key = FileBlockRequest.BlockKey.GetHash(); 

	BlockToReplace->LruPrev->LruNext = BlockToReplace->LruNext;
	BlockToReplace->LruNext->LruPrev = BlockToReplace->LruPrev;

	BlockToReplace->LruPrev = &CacheLruHead;
	BlockToReplace->LruNext = CacheLruHead.LruNext;

	BlockToReplace->LruPrev->LruNext = BlockToReplace;
	BlockToReplace->LruNext->LruPrev = BlockToReplace;

	check(BlockToReplace->Buffer);
	FMemory::Memcpy(BlockToReplace->Buffer, FileBlockRequest.Buffer, CacheBlockSize);
	CachedBlocks.Add(BlockToReplace->Key, BlockToReplace);
	Stats.OnFileBlockCacheStore(CacheBlockSize);
}

////////////////////////////////////////////////////////////////////////////////
FPlatformIoDispatcherRequestMgr::FPlatformIoDispatcherRequestMgr()
{
}

FIoPlatformReadRequest& FPlatformIoDispatcherRequestMgr::CreateScatterGatherRequest(
	FIoFileReadRequestCompleted&& OnCompleted,
	FIoBuffer& Dst,
	uint64 DstSize,
	void* UserData,
	uint32 FileId)
{
	FIoPlatformReadRequest* ReadRequest = ReadRequestAllocator.Construct(
		MoveTemp(OnCompleted),
		Dst,
		DstSize,
		UserData,
		FileId);

	return *ReadRequest;
}

FIoPlatformReadRequest& FPlatformIoDispatcherRequestMgr::CreateDirectReadRequest(
	FIoFileReadRequestCompleted&& OnCompleted,
	FIoBuffer& Dst,
	uint64 DstSize,
	uint64 FileOffset,
	void* UserData)
{
	FIoPlatformReadRequest* ReadRequest = ReadRequestAllocator.Construct(
		MoveTemp(OnCompleted),
		Dst,
		DstSize,
		FileOffset,
		UserData);

	return *ReadRequest;
}

bool FPlatformIoDispatcherRequestMgr::TryCancelReadRequest(FIoPlatformReadRequest& ReadRequest, bool& bAnyBlockCancelled)
{
	bAnyBlockCancelled			= false;
	bool bCompleteReadRequest	= false;

	if (ReadRequest.IsScatterGather())
	{
		if (ReadRequest.ScatterGather.RemainingBlocks.load(std::memory_order_seq_cst) == 0)
		{
			return bCompleteReadRequest;
		}

		bCompleteReadRequest = true;
		for (FIoFileBlockLink* Link = &ReadRequest.ScatterGather.FileBlockLink; Link != nullptr; Link = Link->NextLink)
		{
			check(Link->FileBlockCount > 0);
			for (uint32 Idx = 0; Idx < Link->FileBlockCount; ++Idx)
			{
				// The file block request is removed and destroyed after reading from the filesystem is completed
				const FIoBlockKey FileBlockKey			= FIoBlockKey(ReadRequest.ScatterGather.FileId, Link->FileBlockIds[Idx]);
				FIoFileBlockRequest* FileBlockRequest	= FileBlockLookup.FindRef(FileBlockKey);
				if (FileBlockRequest == nullptr || FileBlockRequest->QueueStatus == FIoFileBlockRequest::EQueueStatus::Dequeued)
				{
					// Only cancel the read request if all file blocks can be found
					bCompleteReadRequest = false;
					continue;
				}

				// Cancel the file block request if all encoded block requests has been cancelled
				bool bCancelFileBlock = true;
				for (FIoEncodedBlockRequest* EncodedBlockRequest : FileBlockRequest->EncodedBlockRequests)
				{
					bool bCancelEncodedBlock = true;
					for (FIoEncodedBlockRequest::FScatterTarget& ScatterTarget : EncodedBlockRequest->ScatterTargets)
					{
						if (ScatterTarget.Request == &ReadRequest)
						{
							ScatterTarget.SizeInBlock = 0;
						}
						else if (ScatterTarget.SizeInBlock > 0)
						{
							bCancelEncodedBlock = false;
							bCancelFileBlock	= false;
						}
					}
					if (bCancelEncodedBlock)
					{
						EncodedBlockRequest->ErrorCode = EIoErrorCode::Cancelled;
						EncodedBlockLookup.Remove(EncodedBlockRequest->BlockKey);
					}
				}

				if (bCancelFileBlock)
				{
					FileBlockRequest->ErrorCode = EIoErrorCode::Cancelled;
					FileBlockLookup.Remove(FileBlockRequest->BlockKey);
					bAnyBlockCancelled = true;
				}
			}
		}
	}

	return bCompleteReadRequest;
}

bool FPlatformIoDispatcherRequestMgr::TryCancelAllReadRequests(FIoFileHandle FileHandle)
{
	// Assumes file queue lock
	TArray<FIoFileBlockRequest*> BlocksToCancel;
	for (TTuple<FIoBlockKey, FIoFileBlockRequest*>& Kv : FileBlockLookup)
	{
		FIoFileBlockRequest* FileBlockRequest = Kv.Value;
		if (FileBlockRequest->FileHandle == FileHandle && FileBlockRequest->QueueStatus == FIoFileBlockRequest::EQueueStatus::Enqueued)
		{
			BlocksToCancel.Add(FileBlockRequest);
		}
	}
	
	for (FIoFileBlockRequest* FileBlockRequest : BlocksToCancel)
	{
		for (FIoEncodedBlockRequest* EncodedBlockRequest : FileBlockRequest->EncodedBlockRequests)
		{
			for (FIoEncodedBlockRequest::FScatterTarget& ScatterTarget : EncodedBlockRequest->ScatterTargets)
			{
				ScatterTarget.SizeInBlock = 0;
			}

			EncodedBlockRequest->ErrorCode = EIoErrorCode::Cancelled;
			EncodedBlockLookup.Remove(EncodedBlockRequest->BlockKey);
		}

		FileBlockRequest->ErrorCode = EIoErrorCode::Cancelled;
		FileBlockLookup.Remove(FileBlockRequest->BlockKey);
	}

	return BlocksToCancel.IsEmpty() == false;
}

void FPlatformIoDispatcherRequestMgr::Destroy(FIoPlatformReadRequest& ReadRequest)
{
	check(ReadRequest.RefCount > 0);
	if (--ReadRequest.RefCount == 0)
	{
		if (ReadRequest.IsScatterGather())
		{
			Destroy(ReadRequest.ScatterGather.FileBlockLink.NextLink);
		}
		ReadRequestAllocator.Destroy(&ReadRequest);
	}
}

FIoFileBlockRequest& FPlatformIoDispatcherRequestMgr::GetOrCreateFileBlockRequest(FIoPlatformReadRequest& ReadRequest, FIoBlockKey BlockKey, bool& bCreated)
{
	check(BlockKey.IsValid());
	if (FIoFileBlockRequest* Existing = FileBlockLookup.FindRef(BlockKey))
	{
		AddToLink(&ReadRequest.ScatterGather.FileBlockLink, BlockKey);
		bCreated = false;
		return *Existing;
	}

	FIoFileBlockRequest* Request	= FileBlockAllocator.Construct();
	Request->BlockKey				= BlockKey;
	bCreated						= true;

	AddToLink(&ReadRequest.ScatterGather.FileBlockLink, BlockKey);
	FileBlockLookup.Add(BlockKey, Request);

	return *Request;
}

FIoFileBlockRequest& FPlatformIoDispatcherRequestMgr::CreateFileBlockRequest()
{
	return *FileBlockAllocator.Construct();
}

FIoFileBlockRequest* FPlatformIoDispatcherRequestMgr::GetFileBlockRequest(FIoBlockKey BlockKey)
{
	return FileBlockLookup.FindRef(BlockKey);
}

void FPlatformIoDispatcherRequestMgr::GetFileBlockRequests(FIoPlatformReadRequest& ReadRequest, FTempArray<FIoFileBlockRequest*>& OutRequests)
{
	for (FIoFileBlockLink* Link = &ReadRequest.ScatterGather.FileBlockLink; Link != nullptr; Link = Link->NextLink)
	{
		check(Link->FileBlockCount > 0);
		for (uint32 Idx = 0; Idx < Link->FileBlockCount; ++Idx)
		{
			const FIoBlockKey FileBlockKey = FIoBlockKey(ReadRequest.ScatterGather.FileId, Link->FileBlockIds[Idx]);
			if (FIoFileBlockRequest* FileBlockRequest = FileBlockLookup.FindRef(FileBlockKey))
			{
				OutRequests.Add(FileBlockRequest);
			}
		}
	}
}

void FPlatformIoDispatcherRequestMgr::AddFileBlockRequest(FIoPlatformReadRequest& ReadRequest, FIoFileBlockRequest& FileBlockRequest)
{
	AddToLink(&ReadRequest.ScatterGather.FileBlockLink, FileBlockRequest.BlockKey);
}

void FPlatformIoDispatcherRequestMgr::AddFileBlockRequest(FIoPlatformReadRequest& ReadRequest, FIoBlockKey FileBlockKey)
{
	AddToLink(&ReadRequest.ScatterGather.FileBlockLink, FileBlockKey);
}

FIoEncodedBlockRequest& FPlatformIoDispatcherRequestMgr::GetOrCreateEncodedBlockRequest(FIoBlockKey BlockKey, bool& bCreated)
{
	check(BlockKey.IsValid());
	if (FIoEncodedBlockRequest* Existing = EncodedBlockLookup.FindRef(BlockKey))
	{
		bCreated = false;
		return *Existing;
	}

	FIoEncodedBlockRequest* Request	= EncodedBlockAllocator.Construct();
	Request->BlockKey				= BlockKey;
	bCreated						= true;

	EncodedBlockLookup.Add(BlockKey, Request);
	return *Request;
}

void FPlatformIoDispatcherRequestMgr::Remove(FIoEncodedBlockRequest& Request)
{
	// Only remove non cancelled blocks
	if (Request.ErrorCode != EIoErrorCode::Cancelled)
	{
		EncodedBlockLookup.Remove(Request.BlockKey);
		if (EncodedBlockLookup.IsEmpty())
		{
			EncodedBlockLookup.Empty(128);
		}
	}
}

void FPlatformIoDispatcherRequestMgr::Destroy(FIoFileBlockRequest& Request)
{
	// Only remove non cancelled blocks
	if (Request.ErrorCode != EIoErrorCode::Cancelled)
	{
		FileBlockLookup.Remove(Request.BlockKey);
		if (FileBlockLookup.IsEmpty())
		{
			FileBlockLookup.Empty(128);
		}
	}
	FileBlockAllocator.Destroy(&Request);
}

void FPlatformIoDispatcherRequestMgr::Destroy(FIoEncodedBlockRequest& Request)
{
	if (Request.FileBlockCount > 1 && Request.EncodedData != nullptr)
	{
		FMemory::Free(Request.EncodedData);
	}
	EncodedBlockAllocator.Destroy(&Request);
}

void FPlatformIoDispatcherRequestMgr::Destroy(FIoFileBlockLink* Link)
{
	while (Link != nullptr)
	{
		FIoFileBlockLink* ToDestroy = Link;
		Link = Link->NextLink;
		FileBlockLinkAllocator.Destroy(ToDestroy);
	}
}

void FPlatformIoDispatcherRequestMgr::AddToLink(FIoFileBlockLink* Link, FIoBlockKey FileBlockKey)
{
	check(FileBlockKey.IsValid());
	check(Link != nullptr);

	for(;;)
	{
		for(uint32 Idx = 0; Idx < Link->FileBlockCount; ++Idx)
		{
			if (Link->FileBlockIds[Idx] == FileBlockKey.GetBlockId())
			{
				return;
			}
		}

		if (Link->FileBlockCount < FIoFileBlockLink::MaxFileCount)
		{
			check(Link->NextLink == nullptr);
			Link->FileBlockIds[Link->FileBlockCount++] = FileBlockKey.GetBlockId();
			return;
		}

		if (Link->NextLink == nullptr)
		{
			Link->NextLink = FileBlockLinkAllocator.Construct();
		}
		Link = Link->NextLink;
	}
}

////////////////////////////////////////////////////////////////////////////////
class FIoServiceThread final
	: public FRunnable
{
public:
	typedef bool(FPlatformIoDispatcherBase::* FThreadEntry)();

	FIoServiceThread(FPlatformIoDispatcherBase* InDispatcher, FThreadEntry InThreadEntry, const TCHAR* InName, EThreadPriority InPriority)
		: Dispatcher(InDispatcher)
		, ThreadEntry(InThreadEntry)
		, Thread(FRunnableThread::Create(this, InName, 0, InPriority))
	{ }

	virtual void	Stop() override	{ bStopRequested.store(true, std::memory_order_seq_cst); WakeUpEvent->Trigger(); }
	void			WakeUp() const	{ WakeUpEvent->Trigger(); }
	void			Join() const	{ Thread->WaitForCompletion(); }

private:
	virtual uint32	Run() override;

	FPlatformIoDispatcherBase*		Dispatcher;
	FThreadEntry					ThreadEntry;
	FEventRef						WakeUpEvent;
	std::atomic_bool				bStopRequested{false};
	TUniquePtr<FRunnableThread>		Thread;
};

////////////////////////////////////////////////////////////////////////////////
uint32 FIoServiceThread::Run() 
{
	for (;;)
	{
		while((Dispatcher->*ThreadEntry)())
		{ }

		if (bStopRequested.load(std::memory_order_relaxed))
		{
			break;
		}
		WakeUpEvent->Wait();
	}

	return 0;
}

////////////////////////////////////////////////////////////////////////////////
FPlatformIoDispatcherBase::FPlatformIoDispatcherBase(FPlatformIoDispatcherCreateParams&& Params)
	: FileBlockMemoryPool(Stats)
	, IoQueue(Stats)
	, FileBlockCache(Stats)
{
	bMultithreaded = Params.bMultithreaded;
}

FPlatformIoDispatcherBase::~FPlatformIoDispatcherBase()
{
	check(bStopRequested);
	check(Thread.IsValid() == false);
	if (SupportThread.IsValid())
	{
		SupportThread->Stop();
		SupportThread->Join();
	}
}

bool FPlatformIoDispatcherBase::Init()
{
	return true;
}

uint32 FPlatformIoDispatcherBase::Run()
{
	return OnIoThreadEntry();
}

void FPlatformIoDispatcherBase::Stop()
{
	bStopRequested = true;
	if (SupportThread.IsValid())
	{
		SupportThread->Stop();
	}
}

FIoStatus FPlatformIoDispatcherBase::Initialize()
{
	FIoStatus Status = OnInitialize();
	if (Status.IsOk() && bMultithreaded)
	{
		Thread.Reset(FRunnableThread::Create(this, TEXT("IoService #1"), 0, TPri_AboveNormal));
		SupportThread = MakeUnique<FIoServiceThread>(this, &FPlatformIoDispatcherBase::ProcessBlocks, TEXT("IoService #2"), TPri_BelowNormal);
	}

	return Status;
}

FIoFileReadRequest FPlatformIoDispatcherBase::ScatterGather(FIoScatterGatherRequestParams&& Params, FIoFileReadRequestCompleted&& OnCompleted)
{
	FIoFileReadRequest		RequestHandle;
	FIoFileBlockRequestList CreatedFileBlockRequests;
	bool					bReprioritizeReadQueue = false;
	{
		TUniqueLock Lock(RequestMgr);

		const FIoPlatformFileInfo FileInfo = GetPlatformFileInfo(Params.FileHandle);
		check(FileInfo.FileId > 0);

		FIoPlatformReadRequest& ReadRequest = RequestMgr.CreateScatterGatherRequest(
			MoveTemp(OnCompleted),
			Params.Destination,
			Params.DestinationSize,
			Params.UserData,
			FileInfo.FileId);

		// The caller and the dispatcher has shared ownership of the read request
		++ReadRequest.RefCount;
		RequestHandle = FIoFileReadRequest(UPTRINT(&ReadRequest));

		for (FIoScatterGatherRequestParams::FScatterParams& ScatterParams : Params.Params)
		{
			const FIoBlockKey			EncodedBlockKey				= FIoBlockKey(FileInfo.FileId, ScatterParams.BlockIndex);
			bool						bEncodedBlockRequestCreated	= false;
			FIoEncodedBlockRequest&		BlockRequest				= RequestMgr.GetOrCreateEncodedBlockRequest(EncodedBlockKey, bEncodedBlockRequestCreated);

			// Scatter info from the encoded block to destination buffer
			BlockRequest.ScatterTargets.Add(FIoEncodedBlockRequest::FScatterTarget
			{
				.Request		= &ReadRequest,
				.OffsetInDst	= ScatterParams.DestinationOffset,
				.OffsetInBlock	= IntCastChecked<uint32>(ScatterParams.ScatterOffset),
				.SizeInBlock	= IntCastChecked<uint32>(ScatterParams.ScatterSize)
			});
			ReadRequest.ScatterGather.RemainingBlocks.fetch_add(1, std::memory_order_relaxed);

			// The encoded/compressed block size are always aligned to AES block size typically 16B
			const uint64 EncodedBlockFileSize	= Align(ScatterParams.BlockCompressedSize, FAES::AESBlockSize);
			const uint64 FirstFileBlockIndex	= ScatterParams.BlockFileOffset / FileBlockSize;
			const uint64 LastFileBlockIndex		= (ScatterParams.BlockFileOffset + EncodedBlockFileSize - 1) / FileBlockSize;
			check(FirstFileBlockIndex < MAX_uint32);
			check(LastFileBlockIndex < MAX_uint32);

			if (bEncodedBlockRequestCreated == false)
			{
				for (uint64 FileBlockIndex = FirstFileBlockIndex; FileBlockIndex <= LastFileBlockIndex; ++FileBlockIndex)
				{
					const FIoBlockKey FileBlockKey			= FIoBlockKey(FileInfo.FileId, IntCastChecked<uint32>(FileBlockIndex));
					FIoFileBlockRequest* FileBlockRequest	= RequestMgr.GetFileBlockRequest(FileBlockKey);

					if (FileBlockRequest != nullptr && Params.Priority > FileBlockRequest->Priority)
					{
						FileBlockRequest->Priority	= Params.Priority;
						bReprioritizeReadQueue 		= true;
					}
					RequestMgr.AddFileBlockRequest(ReadRequest, FileBlockKey);
				}

				continue;
			}

			BlockRequest.FileOffset				= ScatterParams.BlockFileOffset;
			BlockRequest.EncryptionKey			= ScatterParams.EncryptionKey; 
			BlockRequest.BlockHash				= ScatterParams.BlockHash; 
			BlockRequest.BlockCompressedSize	= ScatterParams.BlockCompressedSize;
			BlockRequest.BlockUncompressedSize	= ScatterParams.BlockUncompresedSize; 
			BlockRequest.CompressionMethod		= ScatterParams.CompressionMethod;

			for (uint64 FileBlockIndex = FirstFileBlockIndex; FileBlockIndex <= LastFileBlockIndex; ++FileBlockIndex)
			{
				check(FileInfo.FileSize > 0);

				bool bFileBlockRequestCreated			= false;
				const FIoBlockKey FileBlockKey			= FIoBlockKey(FileInfo.FileId, IntCastChecked<uint32>(FileBlockIndex));
				FIoFileBlockRequest& FileBlockRequest	= RequestMgr.GetOrCreateFileBlockRequest(ReadRequest, FileBlockKey, bFileBlockRequestCreated);

				FileBlockRequest.EncodedBlockRequests.Add(&BlockRequest);

				if (bFileBlockRequestCreated)
				{
					const uint64 FileBlockOffset	= FileBlockIndex * FileBlockSize;
					const uint64 LocalFileBlockSize	= FMath::Min(FileBlockOffset + FileBlockSize, uint64(FileInfo.FileSize)) - FileBlockOffset;

					FileBlockRequest.FileHandle		= Params.FileHandle;
					FileBlockRequest.FileOffset		= FileBlockOffset;
					FileBlockRequest.FileSize		= FileInfo.FileSize;
					FileBlockRequest.Size			= LocalFileBlockSize;
					FileBlockRequest.Priority		= Params.Priority;

					CreatedFileBlockRequests.AddTail(&FileBlockRequest);
				}
				else if (Params.Priority > FileBlockRequest.Priority)
				{
					FileBlockRequest.Priority	= Params.Priority;
					bReprioritizeReadQueue		= true;
				}

				FileBlockRequest.BytesUsed += 
					uint32(FMath::Min(BlockRequest.FileOffset + Align(BlockRequest.BlockCompressedSize, FAES::AESBlockSize), FileBlockRequest.FileOffset + FileBlockRequest.FileSize) -
						   FMath::Max(BlockRequest.FileOffset, FileBlockRequest.FileOffset));

				check(BlockRequest.RemainingFileBlocks < MAX_uint8);
				check(BlockRequest.FileBlockCount < MAX_uint8);
				BlockRequest.RemainingFileBlocks++;
				BlockRequest.FileBlockCount++;
			}
		}
	}
	{
		TUniqueLock QueueLock(IoQueue);
		if (bReprioritizeReadQueue)
		{
			IoQueue.Reprioritize();
		}
		IoQueue.Enqueue(MoveTemp(CreatedFileBlockRequests));
	}
	OnWakeUp();

	return RequestHandle;
}

FIoFileReadRequest FPlatformIoDispatcherBase::ReadDirect(FIoDirectReadRequestParams&& Params, FIoFileReadRequestCompleted&& OnCompleted)
{
	return FIoFileReadRequest();
}

void FPlatformIoDispatcherBase::CancelRequest(FIoFileReadRequest Request)
{
	if (Request.IsValid() == false)
	{
		UE_LOGF(LogPlatformIoDispatcher, Warning, "Trying to cancel an invalid file read request");
		return;
	}

	FIoPlatformReadRequest& ReadRequest = *reinterpret_cast<FIoPlatformReadRequest*>(Request.Value());
	if (ReadRequest.ScatterGather.RemainingBlocks.load(std::memory_order_relaxed) == 0)
	{
		return;
	}

	EIoErrorCode Expected = EIoErrorCode::Ok;
	if (ReadRequest.ErrorCode.compare_exchange_weak(Expected, EIoErrorCode::Cancelled) == false)
	{
		return;
	}

	FIoFileReadRequestCompleted OnCompleted; 
	{
		TUniqueLock MgrLock(RequestMgr);
		TUniqueLock QueueLock(IoQueue);

		bool bAnyBlockCancelled = false;
		if (RequestMgr.TryCancelReadRequest(ReadRequest, bAnyBlockCancelled))
		{
			OnCompleted = MoveTemp(ReadRequest.OnCompleted);
		}
		if (bAnyBlockCancelled)
		{
			IoQueue.ReprioritizeCancelled();
		}
	}

	if (OnCompleted)
	{
		OnCompleted(FIoFileReadResult
		{
			.UserData	= ReadRequest.UserData,
			.ErrorCode	= EIoErrorCode::Cancelled 
		});
	}
}

void FPlatformIoDispatcherBase::CancelAllRequests(FIoFileHandle FileHandle)
{
	if (FileHandle.IsValid() == false)
	{
		UE_LOGF(LogPlatformIoDispatcher, Warning, "Trying to cancel I/O requests for an invalid file");
		return;
	}

	{
		TUniqueLock MgrLock(RequestMgr);
		TUniqueLock QueueLock(IoQueue);
		if (RequestMgr.TryCancelAllReadRequests(FileHandle))
		{
			IoQueue.ReprioritizeCancelled();
		}
	}
}

void FPlatformIoDispatcherBase::UpdatePriority(FIoFileReadRequest Request, int32 NewPriority)
{
	if (Request.IsValid() == false)
	{
		UE_LOGF(LogPlatformIoDispatcher, Warning, "Trying to update priority for an invalid file read request");
		return;
	}

	FIoPlatformReadRequest& ReadRequest = *reinterpret_cast<FIoPlatformReadRequest*>(Request.Value());
	bool bReprioritize = false;
	{
		TUniqueLock MgrLock(RequestMgr);
		TUniqueLock QueueLock(IoQueue);

		FTempArray<FIoFileBlockRequest*> FileBlockRequests;
		RequestMgr.GetFileBlockRequests(ReadRequest, FileBlockRequests);

		for (FIoFileBlockRequest* FileBlockRequest : FileBlockRequests)
		{
			if (NewPriority > FileBlockRequest->Priority)
			{
				FileBlockRequest->Priority	= NewPriority;
				bReprioritize				= true;
			}
		}
		if (bReprioritize)
		{
			IoQueue.Reprioritize();
		}
	}
}

void FPlatformIoDispatcherBase::DeleteRequest(FIoFileReadRequest Request)
{
	if (Request.IsValid())
	{
		FIoPlatformReadRequest* ReadRequest = reinterpret_cast<FIoPlatformReadRequest*>(Request.Value());
		TUniqueLock Lock(RequestMgr);
		RequestMgr.Destroy(*ReadRequest);
	}
}

FIoStatus FPlatformIoDispatcherBase::OnInitialize()
{
	return FIoStatus::Ok;
}

uint32 FPlatformIoDispatcherBase::GetNextFileId()
{
	const uint32 FileId = NextFileId.fetch_add(1);
	if (FileId > 0)
	{
		return FileId;
	}
	return NextFileId.fetch_add(1);
}

void FPlatformIoDispatcherBase::EnqueueCompletedFileBlock(FIoFileBlockRequest& FileBlockRequest)
{
	if (bMultithreaded)
	{
		{
			TUniqueLock Lock(FileBlockMutex);
			FileBlocksToComplete.AddTail(&FileBlockRequest);
		}
		SupportThread->WakeUp();
	}
	else
	{
		FileBlocksToComplete.AddTail(&FileBlockRequest);
		while (ProcessBlocks())
		{ }
	}
}

void FPlatformIoDispatcherBase::CompleteFileBlock(FIoFileBlockRequest& FileBlockRequest)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(CompleteeFileBlock);
	check(IsIoErrorAnyOf(FileBlockRequest.ErrorCode, EIoErrorCode::Ok, EIoErrorCode::Cancelled, EIoErrorCode::ReadError));

	const EIoErrorCode							FileBlockErrorCode = FileBlockRequest.ErrorCode;
	const uint64								FileBlockOffset = FileBlockRequest.FileOffset;
	FIoBufferHandle								FileBlockBufferHandle = FileBlockRequest.BufferHandle;
	FMutableMemoryView							FileBlockBufferView(FileBlockRequest.Buffer, FileBlockRequest.Size);
	FIoFileBlockRequest::FEncodedBlocksArray	EncodedBlockRequests;

	Stats.OnFileBlockCompleted(FileBlockRequest);
	{
		TUniqueLock Lock(RequestMgr);
		EncodedBlockRequests = MoveTemp(FileBlockRequest.EncodedBlockRequests);
		RequestMgr.Destroy(FileBlockRequest);
	}

	FIoEncodedBlockRequestList	CancelledOrFailedBlocks;
	for (FIoEncodedBlockRequest* EncodedBlockRequest : EncodedBlockRequests)
	{
		check(IsIoErrorAnyOf(EncodedBlockRequest->ErrorCode, EIoErrorCode::Unknown, EIoErrorCode::Cancelled, EIoErrorCode::ReadError));
		check(EncodedBlockRequest->RemainingFileBlocks > 0);

		if (FileBlockErrorCode != EIoErrorCode::Ok && EncodedBlockRequest->ErrorCode == EIoErrorCode::Unknown)
		{
			EncodedBlockRequest->ErrorCode = FileBlockErrorCode;
		}

		if (EncodedBlockRequest->ErrorCode == EIoErrorCode::Unknown)
		{
			const uint64 EncodedBlockFileSize	= Align(EncodedBlockRequest->BlockCompressedSize, FAES::AESBlockSize);
			const int64 OffsetInFileBlock		= EncodedBlockRequest->FileOffset - FileBlockOffset;

			// Is the encoded block crossing file block boundaries
			if (EncodedBlockRequest->FileBlockCount > 1)
			{
				// When crossing file blocks we need to allocate a temp buffer to hold the data
				if (EncodedBlockRequest->EncodedData == nullptr)
				{
					EncodedBlockRequest->EncodedData = FMemory::Malloc(EncodedBlockFileSize);
				}

				FMemoryView FileBlock = FileBlockBufferView;
				FMutableMemoryView EncodedBlock(EncodedBlockRequest->EncodedData, EncodedBlockFileSize);

				if (OffsetInFileBlock < 0)
				{
					// Second or third or ... 
					EncodedBlock.RightChopInline(FMath::Abs(OffsetInFileBlock));
					FileBlock.LeftInline(EncodedBlock.GetSize());
				}
				else
				{
					// First file block
					FileBlock.MidInline(OffsetInFileBlock, EncodedBlock.GetSize());
				}
				EncodedBlock.CopyFrom(FileBlock);
			}
			else
			{
				// When the encoded block is completely wihtin the file block we use and keep a reference to the file block memory
				check(OffsetInFileBlock >= 0);
				EncodedBlockRequest->FileBufferHandle	= FileBlockBufferHandle; 
				EncodedBlockRequest->EncodedData		= FileBlockBufferView.Mid(OffsetInFileBlock, EncodedBlockFileSize).GetData();
				FileBlockMemoryPool.AddRef(EncodedBlockRequest->FileBufferHandle);
			}
		}

		if (--EncodedBlockRequest->RemainingFileBlocks == 0)
		{
			TUniqueLock Lock(RequestMgr);
			RequestMgr.Remove(*EncodedBlockRequest);
			if (EncodedBlockRequest->ErrorCode == EIoErrorCode::Unknown)
			{
				BlocksToDecode.AddTail(EncodedBlockRequest);
				Stats.OnDecodeRequestEnqueued(*EncodedBlockRequest);
			}
			else
			{
				check(IsIoErrorAnyOf(EncodedBlockRequest->ErrorCode, EIoErrorCode::Cancelled, EIoErrorCode::ReadError));
				CancelledOrFailedBlocks.AddTail(EncodedBlockRequest);
			}
		}
	}

	if (FileBlockMemoryPool.Free(FileBlockBufferHandle))
	{
		OnWakeUp();
	}

	if (CancelledOrFailedBlocks.IsEmpty() == false)
	{
		FIoPlatformReadRequestList CompletedReadRequests;
		for (FIoEncodedBlockRequest& ToComplete : CancelledOrFailedBlocks)
		{
			for (FIoEncodedBlockRequest::FScatterTarget& ScatterTarget : ToComplete.ScatterTargets)
			{
				ScatterTarget.Request->ErrorCode = ToComplete.ErrorCode;
			}
			CompleteDecodedBlock(ToComplete, CompletedReadRequests);
		}

		{
			TUniqueLock Lock(RequestMgr);
			while (FIoPlatformReadRequest* ReadRequest = CompletedReadRequests.PopHead())
			{
				RequestMgr.Destroy(*ReadRequest);
			}

			while (FIoEncodedBlockRequest* ToComplete = CancelledOrFailedBlocks.PopHead())
			{
				RequestMgr.Destroy(*ToComplete);
			}
		}
	}
}

void FPlatformIoDispatcherBase::EnqueueBlockToDecode(FIoEncodedBlockRequest& EncodedBlockRequest)
{
	// Cancelled or failed requests should never reach blocks to decode
	check(EncodedBlockRequest.ErrorCode == EIoErrorCode::Unknown);
	check(EncodedBlockRequest.EncodedData != nullptr);
	check(EncodedBlockRequest.DecodedData != nullptr && EncodedBlockRequest.BufferHandle.IsValid());

	auto OnBlockDecoded = [this, &EncodedBlockRequest](FIoChunkBlockDecodeResult&& DecodeResult, FIoChunkBlockDecodeRequest& Next)
	{
		ProcessDecodedBlock(EncodedBlockRequest, MoveTemp(DecodeResult), Next);
	};
	FIoChunkBlockDecoder::Get().Enqueue(
		FIoChunkBlockDecodeRequest
		{
			.Params = FIoChunkBlockDecodingParams 
			{
				.EncryptionKey		= EncodedBlockRequest.EncryptionKey,
				.BlockHash			= EncodedBlockRequest.BlockHash,
				.CompressionFormat	= EncodedBlockRequest.CompressionMethod,
			},
			.EncodedBlock			= FMutableMemoryView(EncodedBlockRequest.EncodedData, EncodedBlockRequest.BlockCompressedSize),
			.DecodedBlock			= FMutableMemoryView(EncodedBlockRequest.DecodedData, EncodedBlockRequest.BlockUncompressedSize),
			.OnDecoded				= MoveTemp(OnBlockDecoded)
		});
}

void FPlatformIoDispatcherBase::ProcessDecodedBlock(FIoEncodedBlockRequest& EncodedBlockRequest, FIoChunkBlockDecodeResult&& Result, FIoChunkBlockDecodeRequest& NextDecodeRequest)
{
	check(Result.ErrorCode == EIoErrorCode::Ok || Result.ErrorCode == EIoErrorCode::SignatureError || Result.ErrorCode == EIoErrorCode::CompressionError);
	EncodedBlockRequest.DecodedData	= Result.DecodedBlock.GetData();
	EncodedBlockRequest.ErrorCode	= Result.ErrorCode;
	check(EncodedBlockRequest.ErrorCode != EIoErrorCode::Unknown);

	ScatterDecodedBlock(EncodedBlockRequest); 

	// Free the file block memory and wake up dispatcher to start the next file read request
	if (FileBlockMemoryPool.Free(EncodedBlockRequest.FileBufferHandle))
	{
		OnWakeUp();
	}

	{
		TUniqueLock Lock(EncodedBlockMutex);
		BlocksToComplete.AddTail(&EncodedBlockRequest);
		Stats.OnDecodeRequestCompleted(EncodedBlockRequest);
	}

	// Wakeup support thread to start next decoding request
	if (SupportThread.IsValid())
	{
		SupportThread->WakeUp();
	}
}

void FPlatformIoDispatcherBase::ScatterDecodedBlock(FIoEncodedBlockRequest& EncodedBlockRequest)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(IoDispatcherScatter);
	check(EncodedBlockRequest.ErrorCode != EIoErrorCode::Unknown);

	FMemoryView DecodedBlock(EncodedBlockRequest.DecodedData, EncodedBlockRequest.BlockUncompressedSize);
	for (FIoEncodedBlockRequest::FScatterTarget& ScatterTarget : EncodedBlockRequest.ScatterTargets)
	{
		FIoPlatformReadRequest& Request = *ScatterTarget.Request;
		check(Request.IsScatterGather());
		check(Request.ScatterGather.RemainingBlocks > 0);

		const bool bScatterBlock = ScatterTarget.SizeInBlock > 0 && EncodedBlockRequest.ErrorCode == EIoErrorCode::Ok;
		if (bScatterBlock)
		{
			check(ScatterTarget.Request->Dst.GetSize() > 0);
			FMutableMemoryView	Dst = Request.Dst.GetMutableView().RightChop(ScatterTarget.OffsetInDst);
			FMemoryView			Src = DecodedBlock.Mid(ScatterTarget.OffsetInBlock, ScatterTarget.SizeInBlock);
			Dst.CopyFrom(Src);
			Stats.OnBytesScattered(ScatterTarget.SizeInBlock);
		}
	}
}

void FPlatformIoDispatcherBase::CompleteDecodedBlock(FIoEncodedBlockRequest& EncodedBlockRequest, FIoPlatformReadRequestList& OutCompletedReadRequests)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(CompleteDecodedBlock);
	for (FIoEncodedBlockRequest::FScatterTarget& ScatterTarget : EncodedBlockRequest.ScatterTargets)
	{
		FIoPlatformReadRequest& Request = *ScatterTarget.Request;
		check(Request.IsScatterGather());
		check(Request.ScatterGather.RemainingBlocks > 0);
		
		if (EncodedBlockRequest.ErrorCode != EIoErrorCode::Ok)
		{
			Request.ErrorCode.store(EncodedBlockRequest.ErrorCode, std::memory_order_relaxed);
			Request.FailedBlockId = EncodedBlockRequest.BlockKey.GetBlockId();
		}
		else
		{
			EIoErrorCode Expected = EIoErrorCode::Ok;
			Request.ErrorCode.compare_exchange_weak(Expected, EncodedBlockRequest.ErrorCode);
		}

		if (1 == Request.ScatterGather.RemainingBlocks.fetch_sub(1, std::memory_order_relaxed))
		{
			FIoFileReadRequestCompleted OnCompleted = MoveTemp(Request.OnCompleted);
			if (OnCompleted)
			{
				OnCompleted(FIoFileReadResult
				{
					.UserData		= Request.UserData,
					.FailedBlockId	= Request.FailedBlockId,
					.ErrorCode		= Request.ErrorCode.load(std::memory_order_relaxed),
				});
			}
			OutCompletedReadRequests.AddTail(&Request);
		}
	}
}

bool FPlatformIoDispatcherBase::ProcessCompletedFileBlocks()
{
	FIoFileBlockRequestList ToComplete;
	{
		UE::TUniqueLock Lock(FileBlockMutex);
		ToComplete				= MoveTemp(FileBlocksToComplete);
		FileBlocksToComplete	= FIoFileBlockRequestList();
	}

	if (ToComplete.IsEmpty())
	{
		return false;
	}

	while (FIoFileBlockRequest* FileBlock = ToComplete.PopHead())
	{
		CompleteFileBlock(*FileBlock);
	}

	return true;
}

bool FPlatformIoDispatcherBase::ProcessBlocksToDecode()
{
	bool bDidSomething = false;
	FIoPlatformReadRequestList	ReadRequestsToDestroy;
	FIoEncodedBlockRequestList	EncodedBlockRequestsToDestroy;

	while (FIoEncodedBlockRequest* NextEncodedBlockRequest = BlocksToDecode.PeekHead())
	{
		check(NextEncodedBlockRequest->ErrorCode == EIoErrorCode::Unknown);
		check(NextEncodedBlockRequest->BufferHandle.IsValid() == false);

		// Try allocate temporary decoding memory for the encoded block
		NextEncodedBlockRequest->DecodedData = ChunkBlockMemoryPool.Alloc(NextEncodedBlockRequest->BlockUncompressedSize, NextEncodedBlockRequest->BufferHandle);
		if (NextEncodedBlockRequest->BufferHandle.IsValid() == false)
		{
			break;
		}

		FIoEncodedBlockRequest* EncodedBlockRequest = BlocksToDecode.PopHead();
		bDidSomething = true;

		// Allocate target memory
		{
			LLM_SCOPE_BYNAME(TEXT("FileSystem/FileIoStore"));

			for (FIoEncodedBlockRequest::FScatterTarget& ScatterTarget : EncodedBlockRequest->ScatterTargets)
			{
				FIoPlatformReadRequest& Request = *ScatterTarget.Request;
				// If the size in block is zero the request was cancelled
				if (ScatterTarget.SizeInBlock > 0)
				{
					check(Request.IsScatterGather());
					if (Request.Dst.GetSize() == 0)
					{
						UE::FInheritedContextScope InheritedContextScope = Request.RestoreInheritedContext();
						Request.Dst = FIoBuffer(ScatterTarget.Request->DstSize);
					}
				}
			}
		}

		const bool bDecodeAndScatterAsync = bMultithreaded &&
			(!EncodedBlockRequest->CompressionMethod.IsNone() ||
			 !EncodedBlockRequest->EncryptionKey.IsEmpty() ||
			 !EncodedBlockRequest->BlockHash.IsEmpty());

		if (bDecodeAndScatterAsync)
		{
			EnqueueBlockToDecode(*EncodedBlockRequest);
		}
		else
		{
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(IoDispatcherDecodeBlock);
				const FIoChunkBlockDecodeResult Result = FIoChunkEncoding::DecodeBlock(
					FIoChunkBlockDecodingParams 
					{
						.EncryptionKey		= EncodedBlockRequest->EncryptionKey,
						.BlockHash			= EncodedBlockRequest->BlockHash,
						.CompressionFormat	= EncodedBlockRequest->CompressionMethod,
					},
					FMutableMemoryView(EncodedBlockRequest->EncodedData, EncodedBlockRequest->BlockCompressedSize),
					FMutableMemoryView(EncodedBlockRequest->DecodedData, EncodedBlockRequest->BlockUncompressedSize));

				EncodedBlockRequest->DecodedData	= Result.DecodedBlock.GetData();
				EncodedBlockRequest->ErrorCode		= Result.ErrorCode;
			}

			ScatterDecodedBlock(*EncodedBlockRequest);
			CompleteDecodedBlock(*EncodedBlockRequest, ReadRequestsToDestroy);
			EncodedBlockRequestsToDestroy.AddTail(EncodedBlockRequest);

			// Free the file block memory and wake up the dispatcher to start the next file read request
			if (FileBlockMemoryPool.Free(EncodedBlockRequest->FileBufferHandle))
			{
				OnWakeUp();
			}

			ChunkBlockMemoryPool.Free(EncodedBlockRequest->BufferHandle);
			EncodedBlockRequest->DecodedData = nullptr;
			Stats.OnDecodeRequestCompleted(*EncodedBlockRequest);
		}
	}

	// Destroy the block request and completed read request(s)
	{
		TUniqueLock Lock(RequestMgr);
		while (FIoPlatformReadRequest* ToDestroy = ReadRequestsToDestroy.PopHead())
		{
			RequestMgr.Destroy(*ToDestroy);
		}

		while (FIoEncodedBlockRequest* ToDestroy = EncodedBlockRequestsToDestroy.PopHead())
		{
			RequestMgr.Destroy(*ToDestroy);
		}
	}

	return bDidSomething;
}

bool FPlatformIoDispatcherBase::ProcessBlocksToComplete()
{
	FIoEncodedBlockRequestList ToComplete;
	{
		TUniqueLock Lock(EncodedBlockMutex);
		ToComplete			= MoveTemp(BlocksToComplete);
		BlocksToComplete	= FIoEncodedBlockRequestList();
	}

	if (ToComplete.IsEmpty())
	{
		return false;
	}

	FIoPlatformReadRequestList CompletedReadRequests;
	for (FIoEncodedBlockRequest& EncodedBlockRequest : ToComplete)
	{
		ChunkBlockMemoryPool.Free(EncodedBlockRequest.BufferHandle);
		EncodedBlockRequest.DecodedData = nullptr;
		CompleteDecodedBlock(EncodedBlockRequest, CompletedReadRequests);
	}

	{
		TUniqueLock Lock(RequestMgr);
		while (FIoPlatformReadRequest* ReadRequest = CompletedReadRequests.PopHead())
		{
			RequestMgr.Destroy(*ReadRequest);
		}

		while (FIoEncodedBlockRequest* EncodedBlockRequest = ToComplete.PopHead())
		{
			RequestMgr.Destroy(*EncodedBlockRequest);
		}
	}

	return true;
}

bool FPlatformIoDispatcherBase::ProcessBlocks()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ProcessIO);

	bool bDidSomething = false;

	bDidSomething |= ProcessCompletedFileBlocks();
	bDidSomething |= ProcessBlocksToComplete();
	bDidSomething |= ProcessBlocksToDecode();

	return bDidSomething;
}

} // namespace UE
