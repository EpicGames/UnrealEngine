// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/InheritedContext.h"
#include "Async/Mutex.h"
#include "HAL/PlatformTime.h"
#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "IO/IoAllocators.h"
#include "IO/IoBuffer.h"
#include "IO/IoChunkEncoding.h"
#include "IO/IoContainers.h"
#include "IO/PlatformIoDispatcher.h"
#include "IO/PlatformIoDispatcherStats.h"
#include "IO/IoStatus.h"
#include "Math/NumericLimits.h"
#include "Memory/MemoryView.h"
#include "Templates/UniquePtr.h"

#include <atomic>

namespace UE
{

////////////////////////////////////////////////////////////////////////////////
template<typename T, SIZE_T Size = 8>
using FTempArray = TArray<T, TInlineAllocator<Size>>;

////////////////////////////////////////////////////////////////////////////////
struct FIoBufferHandle
{
	FIoBufferHandle() = default;

	explicit FIoBufferHandle(int32 InHandle)
		: Handle(InHandle)
	{
		check(InHandle != INDEX_NONE);
	}

	int32	Value() const { return Handle; }
	bool	IsValid() const { return Handle != INDEX_NONE; }

private:
	int32 Handle = INDEX_NONE;
};

////////////////////////////////////////////////////////////////////////////////
struct FIoBlockKey
{
	FIoBlockKey()
		: Hash(0)
	{ }

	explicit FIoBlockKey(uint32 InFileId, uint32 InBlockId)
		: FileId(InFileId)
		, BlockId(InBlockId)
	{ }

	uint64	GetHash() const { return Hash; }
	uint32	GetBlockId() const { return BlockId; }
	bool	IsValid() const { return Hash != 0; }

	friend bool operator==(const FIoBlockKey& A, const FIoBlockKey& B)
	{
		return A.Hash == B.Hash;
	}

	friend uint32 GetTypeHash(const FIoBlockKey& Key)
	{
		return GetTypeHash(Key.Hash);
	}

private:
	union
	{
		struct
		{
			uint32 FileId;
			uint32 BlockId;
		};
		uint64 Hash;
	};
};
static_assert(sizeof(FIoBlockKey) == sizeof(uint64));

////////////////////////////////////////////////////////////////////////////////
struct FIoEncodedBlockRequest
	: public TIntrusiveListElement<FIoEncodedBlockRequest>
{
	struct FScatterTarget
	{
		struct FIoPlatformReadRequest* Request = nullptr;
		uint64 OffsetInDst		= 0;
		uint32 OffsetInBlock	= 0;
		uint32 SizeInBlock		= 0;
	};
	static_assert(sizeof(FScatterTarget) == 24);

	using FScatterTargets = TArray<FScatterTarget, TInlineAllocator<2>>;
	static_assert(sizeof(FScatterTargets) == 64);

	FScatterTargets					ScatterTargets;
	FMemoryView						EncryptionKey;
	FMemoryView						BlockHash;
	FIoBlockKey						BlockKey;
	FIoEncodedBlockRequest*			Next = nullptr;
	void*							EncodedData = nullptr;
	void*							DecodedData = nullptr;
	uint64							FileOffset;
	FIoBufferHandle					FileBufferHandle;
	FIoBufferHandle					BufferHandle;
	FName							CompressionMethod = NAME_None;
	uint32							BlockCompressedSize = 0;
	uint32							BlockUncompressedSize = 0;
	EIoErrorCode					ErrorCode = EIoErrorCode::Unknown;
	uint8							RemainingFileBlocks = 0;
	uint8							FileBlockCount = 0;
};

using FIoEncodedBlockRequestAllocator	= TSingleThreadedSlabAllocator<FIoEncodedBlockRequest, 1024>;
using FIoEncodedBlockRequestList		= TIntrusiveList<FIoEncodedBlockRequest>;

////////////////////////////////////////////////////////////////////////////////
struct FIoFileBlockLink
{
	enum { MaxFileCount = 5 };
	uint32				FileBlockIds[MaxFileCount] = {};
	uint32				FileBlockCount = 0;
	FIoFileBlockLink*	NextLink = nullptr;
};
static_assert(sizeof(FIoFileBlockLink) == 32);
using FIoFileBlockLinkAllocator = TSingleThreadedSlabAllocator<FIoFileBlockLink, 128>;

////////////////////////////////////////////////////////////////////////////////
struct FIoPlatformReadRequest
	: public TIntrusiveListElement<FIoPlatformReadRequest>
	, public FInheritedContextBase
{
	enum class EType : uint8
	{
		ScatterGather,
		DirectRead
	};

	struct FScatterGather
	{
		FIoFileBlockLink		FileBlockLink;
		uint32					FileId;
		std::atomic_uint16_t	RemainingBlocks{0};
	};

	struct FDirectRead
	{
		uint64					FileOffset = MAX_uint64;
	};

	FIoPlatformReadRequest(
		FIoFileReadRequestCompleted&& InOnCompleted,
		FIoBuffer& InDst,
		uint64 InDstSize,
		void* InUserData,
		uint32 InFileId)
			: OnCompleted(MoveTemp(InOnCompleted))
			, ScatterGather()
			, Dst(InDst)
			, DstSize(InDstSize)
			, UserData(InUserData)
			, Type(EType::ScatterGather)
	{
		ScatterGather.FileId = InFileId;
	}

	FIoPlatformReadRequest(
		FIoFileReadRequestCompleted&& InOnCompleted,
		FIoBuffer& InDst,
		uint64 InDstSize,
		uint64 InFileOffset,
		void* InUserData)
			: OnCompleted(MoveTemp(InOnCompleted))
			, DirectRead()
			, Dst(InDst)
			, DstSize(InDstSize)
			, UserData(InUserData)
			, Type(EType::DirectRead)
	{
		DirectRead.FileOffset = InFileOffset;
	}

	bool IsScatterGather() const { return Type == EType::ScatterGather; }
	bool IsDirectRead() const { return Type == EType::DirectRead; }

	FIoFileReadRequestCompleted	OnCompleted;
	union
	{
		FScatterGather			ScatterGather;
		FDirectRead				DirectRead;
	};
	FIoPlatformReadRequest*		Next = nullptr;
	FIoBuffer&					Dst;	
	uint64						DstSize = 0;
	void*						UserData = nullptr;
	uint32						RefCount = 1;
	uint32						FailedBlockId = MAX_uint32;
	std::atomic<EIoErrorCode>	ErrorCode{EIoErrorCode::Ok};
	EType						Type = EType::ScatterGather;
};

using FIoPlatformReadRequestAllocator	= TSingleThreadedSlabAllocator<FIoPlatformReadRequest, 1024>;
using FIoPlatformReadRequestList		= TIntrusiveList<FIoPlatformReadRequest>;

////////////////////////////////////////////////////////////////////////////////
struct FIoFileBlockRequest
	: public TIntrusiveListElement<FIoFileBlockRequest>
{
	enum class EQueueStatus : uint8
	{
		None,
		Enqueued,
		Dequeued
	};

	using							FEncodedBlocksArray = TArray<FIoEncodedBlockRequest*, TInlineAllocator<8>>;

	static uint32					NextSeqNo;

	FEncodedBlocksArray				EncodedBlockRequests;
	FIoFileBlockRequest*			Next = nullptr;
	FIoFileBlockRequest*			Prev = nullptr;
	FIoPlatformReadRequest*			DirectReadRequest = nullptr;
	void*							Buffer = nullptr;
	FIoFileHandle					FileHandle;
	FIoBlockKey						BlockKey;
	uint64							FileOffset = 0;
	int64							FileSize = 0;
	uint64							Size = 0;
	uint64							BytesUsed = 0;
	uint64							TimeCreated = FPlatformTime::Cycles64();
	uint32							SeqNo = NextSeqNo++;
	FIoBufferHandle					BufferHandle;
	EIoErrorCode					ErrorCode = EIoErrorCode::Unknown;
	int32							Priority = 0;
	EQueueStatus					QueueStatus = EQueueStatus::None;
};

using FIoFileBlockRequestAllocator	= TSingleThreadedSlabAllocator<FIoFileBlockRequest, 1024>;
using FIoFileBlockRequestList		= TIntrusiveList<FIoFileBlockRequest>;

////////////////////////////////////////////////////////////////////////////////
struct FIoPlatformFileInfo
{
	int64	FileSize = -1;
	uint32	FileId = 0;
	uint32	CompressionBlockSize = 0;
};

////////////////////////////////////////////////////////////////////////////////
class FIoQueue
{
	using FIoFileBlockRequestDoubleLinkedList = TIntrusiveTwoWayList<FIoFileBlockRequest>;

	struct FFileOffset
	{
		FIoFileHandle	FileHandle;
		uint64			FileOffset = MAX_uint64;
		int32			Priority = 0;
	};

	struct FPrioQueue
	{
		explicit	FPrioQueue(int32 InPriority) : Priority(InPriority) { }
					FPrioQueue(const FPrioQueue&) = delete;
					FPrioQueue(FPrioQueue&&) = default; 

		FPrioQueue& operator=(const FPrioQueue&) = delete;
		FPrioQueue& operator=(FPrioQueue&&) = default; 

		TArray<FIoFileBlockRequest*>		ByOffset;
		FIoFileBlockRequestDoubleLinkedList	BySequence;
		int32								PeekIndex = INDEX_NONE;
		int32								Priority; 
	};

	static FFileOffset ToFileOffset(const FIoFileBlockRequest* Request)
	{
		return FFileOffset
		{
			.FileHandle = Request->FileHandle,
			.FileOffset = Request->FileOffset,
			.Priority	= Request->Priority
		};
	}

	static int32 QueueToPriority(const FPrioQueue& Queue) { return Queue.Priority; }

	static bool FileOffsetLess(const FFileOffset& LHS, const FFileOffset& RHS)
	{
		if (LHS.FileHandle.Value() == RHS.FileHandle.Value())
		{
			return LHS.FileOffset < RHS.FileOffset;
		}
		return LHS.FileHandle.Value() < RHS.FileHandle.Value();
	}

	static bool SeqNoLess(const FIoFileBlockRequest& LHS, const FIoFileBlockRequest& RHS)
	{
		if (LHS.Priority == RHS.Priority)
		{
			return LHS.SeqNo < RHS.SeqNo;
		}
		return LHS.Priority > RHS.Priority;
	}

public:
							FIoQueue(FPlatformIoDispatcherStats& InStats);
	void					Enqueue(FIoFileBlockRequestList&& Requests);
	FIoFileBlockRequest*	Dequeue();
	FIoFileBlockRequest*	Peek();
	void					Reprioritize();
	void					ReprioritizeCancelled() { bReprioritizeCancelled = true; }
	void					SortByOffset(bool bValue) { bSortByOffset = bValue; }

	inline void				Lock() { Mutex.Lock(); }
	inline void				Unlock() { Mutex.Unlock(); }

private:
	void					AddToPrioQueue(FIoFileBlockRequest& Request);
	FIoFileBlockRequest*	GetByOffset(bool bDequeue);
	FIoFileBlockRequest*	GetByOffset(FPrioQueue& PrioQueue, bool bDequeue);
	FIoFileBlockRequest*	GetCancelled(bool bDequeue);

	FPlatformIoDispatcherStats&		Stats;
	TArray<FIoFileBlockRequest*>	Heap;
	TArray<FIoFileBlockRequest*>	CancelledHeap;
	TArray<FPrioQueue>				PrioQueues;
	FFileOffset						LastFileOffset;
	FMutex							Mutex;
	bool							bSortByOffset = false;
	bool							bReprioritizeCancelled = false;
};

////////////////////////////////////////////////////////////////////////////////
class FIoFileBlockMemoryPool
{
public:
					FIoFileBlockMemoryPool(FPlatformIoDispatcherStats& InStats);
	void			Initialize(uint32 FileBlockSize, uint32 PoolSize, uint32 Alignment = 4096);
	void*			Alloc(FIoBufferHandle& OutHandle);
	bool			Free(FIoBufferHandle& Handle);
	void			AddRef(FIoBufferHandle Handle);

private:

	struct FDeleteByFree
	{
		void operator()(void* Ptr) const
		{
			FMemory::Free(Ptr);
		}
	};

	struct FMemoryBlock
	{
		FMemoryBlock*		Next		= nullptr;
		void*				Memory		= nullptr;
		int32				Index		= INDEX_NONE;
		int32				RefCount	= 0;
	};

	FPlatformIoDispatcherStats&			Stats;
	TUniquePtr<void, FDeleteByFree>		BlockMemory;
	TArray<FMemoryBlock>				MemoryBlocks;
	FMemoryBlock*						FreeBlock = nullptr;
	uint32								BlockSize;
	FMutex								Mutex;
};

////////////////////////////////////////////////////////////////////////////////
class FIoChunkBlockMemoryPool
{
public:
						FIoChunkBlockMemoryPool();
	void				Initialize(uint32 MaxBlockCount, uint32 DefaultBlockSize);
	void*				Alloc(uint32 BlockSize, FIoBufferHandle& OutHandle);
	void*				Realloc(FIoBufferHandle Handle, uint32 BlockSize);
	void				Free(FIoBufferHandle& Handle);
	bool				IsEmpty() const { return FreeBlock == nullptr; }

private:
	struct FMemoryBlock
	{
		FMemoryBlock*	Next	= nullptr;
		void*			Memory	= nullptr;
		uint32			Size	= 0;
		int32			Index	= INDEX_NONE; 
	};

	TArray<FMemoryBlock>	MemoryBlocks;
	FMemoryBlock*			FreeBlock = nullptr;
};

////////////////////////////////////////////////////////////////////////////////
class FIoFileBlockCache
{
public:
			FIoFileBlockCache(FPlatformIoDispatcherStats& Stats);
			~FIoFileBlockCache();

	void	Initialize(uint64 CacheBlockSize, uint64 CacheSize);
	bool 	Get(FIoFileBlockRequest& FileBlockRequest);
	void 	Put(FIoFileBlockRequest& FileBlockRequest);

private:
	struct FCachedBlock
	{
		FCachedBlock*	LruPrev = nullptr;
		FCachedBlock* 	LruNext = nullptr;
		uint64			Key		= 0;
		uint8*			Buffer	= nullptr;
	};

	FPlatformIoDispatcherStats&		Stats;
	TUniquePtr<uint8[]>				CacheMemory;
	TMap<uint64, FCachedBlock*>		CachedBlocks;
	FCachedBlock					CacheLruHead;
	FCachedBlock					CacheLruTail;
	uint64							CacheBlockSize = 0;
};

////////////////////////////////////////////////////////////////////////////////
class FPlatformIoDispatcherRequestMgr
{
public:
										FPlatformIoDispatcherRequestMgr();
	FIoPlatformReadRequest&				CreateScatterGatherRequest(
											FIoFileReadRequestCompleted&& OnCompleted,
											FIoBuffer& Dst,
											uint64 DstSize,
											void* UserData,
											uint32 FileId);
	FIoPlatformReadRequest&				CreateDirectReadRequest(
											FIoFileReadRequestCompleted&& OnCompleted,
											FIoBuffer& Dst,
											uint64 DstSize,
											uint64 FileOffset,
											void* UserData);
	bool								TryCancelReadRequest(FIoPlatformReadRequest& ReadRequest, bool& bAnyBlockCancelled);
	bool								TryCancelAllReadRequests(FIoFileHandle FileHandle);
	void								Destroy(FIoPlatformReadRequest& ReadRequest);
	FIoFileBlockRequest&				GetOrCreateFileBlockRequest(FIoPlatformReadRequest& ReadRequest, FIoBlockKey BlockKey, bool& bCreated);
	FIoFileBlockRequest&				CreateFileBlockRequest();
	FIoFileBlockRequest*				GetFileBlockRequest(FIoBlockKey BlockKey);
	void								GetFileBlockRequests(FIoPlatformReadRequest& ReadRequest, FTempArray<FIoFileBlockRequest*>& OutRequests);
	void								AddFileBlockRequest(FIoPlatformReadRequest& ReadRequest, FIoFileBlockRequest& FileBlockRequest);
	void								AddFileBlockRequest(FIoPlatformReadRequest& ReadRequest, FIoBlockKey FileBlockKey);
	FIoEncodedBlockRequest&				GetOrCreateEncodedBlockRequest(FIoBlockKey BlockKey, bool& bCreated);
	void								Remove(FIoEncodedBlockRequest& Request);
	void								Destroy(FIoFileBlockRequest& Request);
	void								Destroy(FIoEncodedBlockRequest& Request);
	void								Destroy(FIoFileBlockLink* Link);

	inline void							Lock() { Mutex.Lock(); }
	inline void							Unlock() { Mutex.Unlock(); }

private:
	void								AddToLink(FIoFileBlockLink* Link, FIoBlockKey FileBlockKey);
	using FFileBlockLookup				= TMap<FIoBlockKey, FIoFileBlockRequest*>;
	using FEncodedBlockLookup			= TMap<FIoBlockKey, FIoEncodedBlockRequest*>;

	FIoPlatformReadRequestAllocator		ReadRequestAllocator;
	FIoFileBlockRequestAllocator		FileBlockAllocator;
	FFileBlockLookup					FileBlockLookup;
	FIoFileBlockLinkAllocator 			FileBlockLinkAllocator;
	FIoEncodedBlockRequestAllocator		EncodedBlockAllocator;
	FEncodedBlockLookup					EncodedBlockLookup;
	UE::FMutex							Mutex;
};

// Forward declare thread helper
class FIoServiceThread;

////////////////////////////////////////////////////////////////////////////////
/// This is a base class for platform specific I/O dispatcher implementations.
/// There are two kinds of I/O requests when reading data from disk. Direct-read requests
/// for platforms with hardware decompression and scatter-gather requests. Scatter-gather
/// requests are issued by specifying information about how the blocks on disk are encoded, i.e.
/// compressed, encrypted and or signed and forms a set of encoded block request. Encoded
/// block requests can be shared between user requests, i.e. if two user requests are reading
/// the same encoded block, the block is decoded once and then scattered to the destination/targets buffers.
/// The encoded block requests are devided into a set of larger file block requests. The
/// size of the file blocks are configured with GIoDispatcherBufferSizeKB (default=256KiB).
/// The maximum number of concurrent file block requests are constrained by the available
/// memory in the file block memory pool configured with GIoDispatcherBufferMemoryMB (default=8MiB).
class FPlatformIoDispatcherBase
	: public FRunnable
	, public IPlatformIoDispatcher
{
public:
	virtual								~FPlatformIoDispatcherBase();

	// FRunnable
	virtual bool						Init() override final;
	virtual uint32						Run() override final;
	virtual void						Stop() override final;

	// IIoPlatformFile
	virtual FIoStatus					Initialize() override final;
	virtual FIoFileReadRequest			ScatterGather(FIoScatterGatherRequestParams&& Params, FIoFileReadRequestCompleted&& OnCompleted) override;
	virtual FIoFileReadRequest			ReadDirect(FIoDirectReadRequestParams&& Params, FIoFileReadRequestCompleted&& OnCompleted) override;
	virtual bool						Tick() override { ensure(!bMultithreaded); return false; }
	virtual void						UpdatePriority(FIoFileReadRequest Request, int32 NewPriority) override;
	virtual void						CancelRequest(FIoFileReadRequest Request) override;
	virtual void						CancelAllRequests(FIoFileHandle FileHandle) override;
	virtual void						DeleteRequest(FIoFileReadRequest Request) override;

protected:
										FPlatformIoDispatcherBase(FPlatformIoDispatcherCreateParams&& Params);
	virtual FIoStatus					OnInitialize();
	virtual uint32						OnIoThreadEntry() = 0;
	virtual void						OnWakeUp() = 0;
	virtual FIoPlatformFileInfo			GetPlatformFileInfo(FIoFileHandle FileHandle) = 0;

	uint32								GetNextFileId();

	void								EnqueueCompletedFileBlock(FIoFileBlockRequest& FileBlockRequest);
	void								CompleteFileBlock(FIoFileBlockRequest& FileBlockRequest);
	void								EnqueueBlockToDecode(FIoEncodedBlockRequest& EncodedBlockRequest);
	void								ProcessDecodedBlock(FIoEncodedBlockRequest& EncodedBlockRequest, FIoChunkBlockDecodeResult&& DecodeResult, FIoChunkBlockDecodeRequest& NextDecodeRequest);
	void								ScatterDecodedBlock(FIoEncodedBlockRequest& EncodedBlockRequest);
	void								CompleteDecodedBlock(FIoEncodedBlockRequest& EncodedBlockRequest, FIoPlatformReadRequestList& OutCompletedReadRequests);

	bool								ProcessCompletedFileBlocks();
	bool								ProcessBlocksToDecode();
	bool								ProcessBlocksToComplete();
	bool								ProcessBlocks();

	FPlatformIoDispatcherStats			Stats;
	TUniquePtr<FRunnableThread>			Thread;
	TUniquePtr<FIoServiceThread>		SupportThread;
	FPlatformIoDispatcherRequestMgr		RequestMgr;
	FIoChunkBlockMemoryPool				ChunkBlockMemoryPool;
	FIoFileBlockMemoryPool				FileBlockMemoryPool;
	FIoQueue							IoQueue;
	FIoFileBlockCache					FileBlockCache;
	FIoFileBlockRequestList				FileBlocksToComplete;
	FIoEncodedBlockRequestList			BlocksToDecode;
	FIoEncodedBlockRequestList			BlocksToComplete;
	UE::FMutex							FileBlockMutex;
	UE::FMutex							EncodedBlockMutex;
	uint32								FileBlockSize = 256 << 10;
	bool								bMultithreaded = false;
	std::atomic_bool					bStopRequested{false};

private:
	std::atomic_uint32_t				NextFileId{1};
};

} // namespace UE 
