// Copyright Epic Games, Inc. All Rights Reserved.

#include "IO/PlatformIoDispatcherStats.h"
#include "IO/PlatformIoDispatcherBase.h"

namespace UE
{

#if UE_PLATFORM_DISPATCHER_STATS_ENABLED
void FPlatformIoDispatcherStats::OnFileBlockRequestEnqueued(FIoFileBlockRequest& FileBlockRequest)
{
	Stats.OnReadRequestsQueued(FileBlockRequest.Size, 1);
}

void FPlatformIoDispatcherStats::OnFileBlockRequestDequeued(FIoFileBlockRequest& FileBlockRequest)
{
}

void FPlatformIoDispatcherStats::OnFileBlockReadStarted(FIoFileBlockRequest& FileBlockRequest)
{
	Stats.OnFilesystemReadStarted(FileBlockRequest.FileHandle.Value(), FileBlockRequest.FileOffset, FileBlockRequest.Size);
}

void FPlatformIoDispatcherStats::OnFileBlockReadCompleted(FIoFileBlockRequest& FileBlockRequest)
{
	Stats.OnFilesystemReadCompleted(FileBlockRequest.FileHandle.Value(), FileBlockRequest.FileOffset, FileBlockRequest.Size);
}

void FPlatformIoDispatcherStats::OnFileBlockCompleted(FIoFileBlockRequest& FileBlockRequest)
{
	Stats.OnReadRequestsCompleted(FileBlockRequest.Size, 1);
}

void FPlatformIoDispatcherStats::OnFileBlockMemoryAllocated(uint32 Size)
{
	Stats.OnBufferAllocated();
}

void FPlatformIoDispatcherStats::OnFileBlockMemoryFreed(uint32 Size)
{
	Stats.OnBufferReleased();
}

void FPlatformIoDispatcherStats::OnFileBlockCacheHit(uint64 Size)
{
	Stats.OnBlockCacheHit(Size);
}

void FPlatformIoDispatcherStats::OnFileBlockCacheMiss(uint64 Size)
{
	Stats.OnBlockCacheMiss(Size);
}

void FPlatformIoDispatcherStats::OnFileBlockCacheStore(uint64 Size)
{
	Stats.OnBlockCacheStore(Size);
}

void FPlatformIoDispatcherStats::OnDecodeRequestEnqueued(FIoEncodedBlockRequest& EncodedBlockRequest)
{
	Stats.OnDecompressQueued(EncodedBlockRequest.BlockCompressedSize, EncodedBlockRequest.BlockUncompressedSize);
}

void FPlatformIoDispatcherStats::OnDecodeRequestCompleted(FIoEncodedBlockRequest& EncodedBlockRequest)
{
	Stats.OnDecompressComplete(EncodedBlockRequest.BlockCompressedSize, EncodedBlockRequest.BlockUncompressedSize);
}

void FPlatformIoDispatcherStats::OnBytesScattered(int64 BytesScattered)
{
	Stats.OnBytesScattered(BytesScattered);
}
#endif

} // namespace UE
