// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IO/IoDispatcherFilesystemStats.h"

#define UE_PLATFORM_DISPATCHER_STATS_ENABLED (COUNTERSTRACE_ENABLED || CSV_PROFILER_STATS)

namespace UE
{

struct FIoFileBlockRequest;
struct FIoEncodedBlockRequest;

////////////////////////////////////////////////////////////////////////////////
#if UE_PLATFORM_DISPATCHER_STATS_ENABLED 
class FPlatformIoDispatcherStats
{
public:
	void OnFileBlockRequestEnqueued(FIoFileBlockRequest& FileBlockRequest);
	void OnFileBlockRequestDequeued(FIoFileBlockRequest& FileBlockRequest);
	void OnFileBlockReadStarted(FIoFileBlockRequest& FileBlockRequest);
	void OnFileBlockReadCompleted(FIoFileBlockRequest& FileBlockRequest);
	void OnFileBlockCompleted(FIoFileBlockRequest& FileBlockRequest);
	void OnFileBlockMemoryAllocated(uint32 Size);
	void OnFileBlockMemoryFreed(uint32 Size);
	void OnFileBlockCacheHit(uint64 Size);
	void OnFileBlockCacheMiss(uint64 Size);
	void OnFileBlockCacheStore(uint64 Size);
	void OnDecodeRequestEnqueued(FIoEncodedBlockRequest& EncodedBlockRequest);
	void OnDecodeRequestCompleted(FIoEncodedBlockRequest& EncodedBlockRequest);
	void OnBytesScattered(int64 BytesScattered);

private:
	FIoDispatcherFilesystemStats Stats;
};
#else

////////////////////////////////////////////////////////////////////////////////
class FPlatformIoDispatcherStats
{
public:
	void OnFileBlockRequestEnqueued(FIoFileBlockRequest& FileBlockRequest) { };
	void OnFileBlockRequestDequeued(FIoFileBlockRequest& FileBlockRequest) { };
	void OnFileBlockReadStarted(FIoFileBlockRequest& FileBlockRequest) { };
	void OnFileBlockReadCompleted(FIoFileBlockRequest& FileBlockRequest) { };
	void OnFileBlockCompleted(FIoFileBlockRequest& FileBlockRequest) { }
	void OnFileBlockMemoryAllocated(uint32 Size) { };
	void OnFileBlockMemoryFreed(uint32 Size) { };
	void OnFileBlockCacheHit(uint64 Size) { }
	void OnFileBlockCacheMiss(uint64 Size) { }
	void OnFileBlockCacheStore(uint64 Size) { }
	void OnDecodeRequestEnqueued(FIoEncodedBlockRequest& EncodedBlockRequest) { }
	void OnDecodeRequestCompleted(FIoEncodedBlockRequest& EncodedBlockRequest) { }
	void OnBytesScattered(int64 BytesScattered) { }
};
#endif

} // namespace UE
