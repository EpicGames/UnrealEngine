// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/CoreMiscDefines.h"

#if USING_INSTRUMENTATION

#include "CoreTypes.h"
#include "Containers/Map.h"

#include "Instrumentation/Defines.h"

// ------------------------------------------------------------------------------
// Allocators (we can't use the Engine default allocators if they're instrumented)
// ------------------------------------------------------------------------------
#if PLATFORM_WINDOWS
class FInstrumentationSafeWinAllocator {
public:
	INSTRUMENTATION_FUNCTION_ATTRIBUTES static void* Alloc(SIZE_T Size);
	INSTRUMENTATION_FUNCTION_ATTRIBUTES static void Free(void* Data);
	INSTRUMENTATION_FUNCTION_ATTRIBUTES static void* AllocWithGuards(SIZE_T Size);
	INSTRUMENTATION_FUNCTION_ATTRIBUTES static void FreeWithGuards(void* Data);
	INSTRUMENTATION_FUNCTION_ATTRIBUTES static void* Realloc(void* Data, SIZE_T Size, SIZE_T PreviousSize);
};

using TInstrumentationSafeBaseAllocator = FInstrumentationSafeWinAllocator;

#define SAFE_OPERATOR_NEW_DELETE() \
	INSTRUMENTATION_FUNCTION_ATTRIBUTES void * operator new(SIZE_T Size)  \
	{ \
		return FInstrumentationSafeWinAllocator::Alloc(Size); \
	} \
	INSTRUMENTATION_FUNCTION_ATTRIBUTES void   operator delete(void* Ptr) \
	{ \
		FInstrumentationSafeWinAllocator::Free(Ptr);  \
	} 

#define SAFE_OPERATOR_NEW_DELETE_WITH_GUARDS() \
	INSTRUMENTATION_FUNCTION_ATTRIBUTES void * operator new(SIZE_T Size)  \
	{ \
		return FInstrumentationSafeWinAllocator::AllocWithGuards(Size); \
	} \
	INSTRUMENTATION_FUNCTION_ATTRIBUTES void   operator delete(void* Ptr) \
	{ \
		FInstrumentationSafeWinAllocator::FreeWithGuards(Ptr);  \
	} 

#else
class TInstrumentationSafeMallocAllocator {
public:
	INSTRUMENTATION_FUNCTION_ATTRIBUTES static void* Realloc(void* Data, SIZE_T Size, SIZE_T PreviousSize)
	{
		return realloc(Data, Size);
	}
	INSTRUMENTATION_FUNCTION_ATTRIBUTES static void Free(void* Data)
	{
		return free(Data);
	}
};

using TInstrumentationSafeBaseAllocator = TInstrumentationSafeMallocAllocator;
#endif

template <int IndexSize = 32, typename TBaseAllocator = TInstrumentationSafeBaseAllocator>
class TInstrumentationSafeAllocator {
public:
	using SizeType = typename TBitsToSizeType<IndexSize>::Type;

private:
	using USizeType = std::make_unsigned_t<SizeType>;

public:
	enum { NeedsElementType = true };
	enum { RequireRangeCheck = true };

	class ForAnyElementType
	{
	public:
		/** Default constructor. */
		INSTRUMENTATION_FUNCTION_ATTRIBUTES ForAnyElementType()
			: Data(nullptr)
		{
		}

		/**
		 * Moves the state of another allocator into this one.
		 * Moves the state of another allocator into this one.  The allocator can be different.
		 *
		 * Assumes that the allocator is currently empty, i.e. memory may be allocated but any existing elements have already been destructed (if necessary).
		 * @param Other - The allocator to move the state from.  This allocator should be left in a valid empty state.
		 */
		INSTRUMENTATION_FUNCTION_ATTRIBUTES FORCEINLINE void MoveToEmpty(ForAnyElementType& Other)
		{
			if (Data)
			{
				TBaseAllocator::Free(Data);
			}

			Data = Other.Data;
			Other.Data = nullptr;
		}

		/** Destructor. */
		INSTRUMENTATION_FUNCTION_ATTRIBUTES FORCEINLINE ~ForAnyElementType()
		{
			if (Data)
			{
				TBaseAllocator::Free(Data);
			}
		}

		// FContainerAllocatorInterface
		INSTRUMENTATION_FUNCTION_ATTRIBUTES FORCEINLINE void* GetAllocation() const
		{
			return Data;
		}

		INSTRUMENTATION_FUNCTION_ATTRIBUTES void ResizeAllocation(SizeType PreviousNumElements, SizeType NumElements, SIZE_T NumBytesPerElement)
		{
			// Avoid calling FMemory::Realloc( nullptr, 0 ) as ANSI C mandates returning a valid pointer which is not what we want.
			if (Data || NumElements)
			{
				static_assert(sizeof(SizeType) <= sizeof(SIZE_T), "SIZE_T is expected to handle all possible sizes");

				// Check for under/overflow
				bool bInvalidResize = NumElements < 0 || NumBytesPerElement < 1 || NumBytesPerElement >(SIZE_T)MAX_int32;
				if constexpr (sizeof(SizeType) == sizeof(SIZE_T))
				{
					bInvalidResize = bInvalidResize || (SIZE_T)(USizeType)NumElements > (SIZE_T)TNumericLimits<SizeType>::Max() / NumBytesPerElement;
				}
				if (UNLIKELY(bInvalidResize))
				{
					UE::Core::Private::OnInvalidSizedHeapAllocatorNum(IndexSize, NumElements, NumBytesPerElement);
				}

				Data = TBaseAllocator::Realloc(Data, NumElements * NumBytesPerElement, PreviousNumElements * NumBytesPerElement);
				if (NumElements)
				{
					checkSlow(Data);
				}
			}
		}
		INSTRUMENTATION_FUNCTION_ATTRIBUTES FORCEINLINE SizeType CalculateSlackReserve(SizeType NumElements, SIZE_T NumBytesPerElement) const
		{
			return DefaultCalculateSlackReserve(NumElements, NumBytesPerElement, true);
		}
		INSTRUMENTATION_FUNCTION_ATTRIBUTES FORCEINLINE SizeType CalculateSlackReserve(SizeType NumElements, SIZE_T NumBytesPerElement, uint32 AlignmentOfElement) const
		{
			return DefaultCalculateSlackReserve(NumElements, NumBytesPerElement, true, (uint32)AlignmentOfElement);
		}
		INSTRUMENTATION_FUNCTION_ATTRIBUTES FORCEINLINE SizeType CalculateSlackShrink(SizeType NumElements, SizeType NumAllocatedElements, SIZE_T NumBytesPerElement) const
		{
			return DefaultCalculateSlackShrink(NumElements, NumAllocatedElements, NumBytesPerElement, true);
		}
		INSTRUMENTATION_FUNCTION_ATTRIBUTES FORCEINLINE SizeType CalculateSlackShrink(SizeType NumElements, SizeType NumAllocatedElements, SIZE_T NumBytesPerElement, uint32 AlignmentOfElement) const
		{
			return DefaultCalculateSlackShrink(NumElements, NumAllocatedElements, NumBytesPerElement, true, (uint32)AlignmentOfElement);
		}
		INSTRUMENTATION_FUNCTION_ATTRIBUTES FORCEINLINE SizeType CalculateSlackGrow(SizeType NumElements, SizeType NumAllocatedElements, SIZE_T NumBytesPerElement) const
		{
			return DefaultCalculateSlackGrow(NumElements, NumAllocatedElements, NumBytesPerElement, true);
		}
		INSTRUMENTATION_FUNCTION_ATTRIBUTES FORCEINLINE SizeType CalculateSlackGrow(SizeType NumElements, SizeType NumAllocatedElements, SIZE_T NumBytesPerElement, uint32 AlignmentOfElement) const
		{
			return DefaultCalculateSlackGrow(NumElements, NumAllocatedElements, NumBytesPerElement, true, (uint32)AlignmentOfElement);
		}

		INSTRUMENTATION_FUNCTION_ATTRIBUTES SIZE_T GetAllocatedSize(SizeType NumAllocatedElements, SIZE_T NumBytesPerElement) const
		{
			return NumAllocatedElements * NumBytesPerElement;
		}

		INSTRUMENTATION_FUNCTION_ATTRIBUTES bool HasAllocation() const
		{
			return !!Data;
		}

		INSTRUMENTATION_FUNCTION_ATTRIBUTES SizeType GetInitialCapacity() const
		{
			return 0;
		}

	private:
		ForAnyElementType(const ForAnyElementType&);
		ForAnyElementType& operator=(const ForAnyElementType&);

		/** A pointer to the container's elements. */
		void* Data;
	};

	template<typename ElementType>
	class ForElementType : public ForAnyElementType
	{
	public:
		/** Default constructor. */
		INSTRUMENTATION_FUNCTION_ATTRIBUTES ForElementType()
		{
		}

		INSTRUMENTATION_FUNCTION_ATTRIBUTES FORCEINLINE ElementType* GetAllocation() const
		{
			return (ElementType*)ForAnyElementType::GetAllocation();
		}
	};
};

template <uint32 NumInlineElements, typename BaseAllocator = TInstrumentationSafeBaseAllocator>
using TInstrumentationSafeInlineAllocator = TSizedInlineAllocator<NumInlineElements, 32, TInstrumentationSafeAllocator<32, BaseAllocator>>;

template <typename ElementType, typename BaseAllocator = TInstrumentationSafeBaseAllocator>
using TSafeArray = TArray<ElementType, TInstrumentationSafeAllocator<32, BaseAllocator>>;

template<
	typename BaseAllocator = TInstrumentationSafeBaseAllocator,
	uint32   AverageNumberOfElementsPerHashBucket = DEFAULT_NUMBER_OF_ELEMENTS_PER_HASH_BUCKET,
	uint32   BaseNumberOfHashBuckets = DEFAULT_BASE_NUMBER_OF_HASH_BUCKETS,
	uint32   MinNumberOfHashedElements = DEFAULT_MIN_NUMBER_OF_HASHED_ELEMENTS
>
class TInstrumentationSafeSparseSetAllocator
{
public:

	/** Computes the number of hash buckets to use for a given number of elements. */
	INSTRUMENTATION_FUNCTION_ATTRIBUTES static FORCEINLINE uint32 GetNumberOfHashBuckets(uint32 NumHashedElements)
	{
		if (NumHashedElements >= MinNumberOfHashedElements)
		{
			return FPlatformMath::RoundUpToPowerOfTwo(NumHashedElements / AverageNumberOfElementsPerHashBucket + BaseNumberOfHashBuckets);
		}

		return 1;
	}

	using InSparseArrayAllocator = TSparseArrayAllocator<TInstrumentationSafeAllocator<32, BaseAllocator>, TInstrumentationSafeInlineAllocator<4, BaseAllocator>>;
	using InHashAllocator = TInstrumentationSafeInlineAllocator<1, BaseAllocator>;

	typedef InSparseArrayAllocator SparseArrayAllocator;
	typedef InHashAllocator        HashAllocator;
	
};

template<typename BaseAllocator = TInstrumentationSafeBaseAllocator>
struct TInstrumentationSafeCompactSetAllocator
{
	using InElementAllocator = TInstrumentationSafeAllocator<32, BaseAllocator>;

	/** CompactSets use raw uint8, so we need extra alignment information for inline allocators */
	template <typename ElementType>
	struct AllocatorAlignment
	{
		static constexpr size_t Value = alignof(typename InElementAllocator::template ForElementType<uint8>);
	};

	template<int32 ElementSize>
	using ElementAllocator = InElementAllocator;
};

#if UE_USE_COMPACT_SET_AS_DEFAULT
template<typename BaseAllocator>
using TInstrumentationSafeSetAllocator = TInstrumentationSafeCompactSetAllocator<BaseAllocator>;
#else
template<typename BaseAllocator>
using TInstrumentationSafeSetAllocator = TInstrumentationSafeSparseSetAllocator<BaseAllocator>;
#endif

template <typename KeyType, typename ValueType, typename BaseAllocator = TInstrumentationSafeBaseAllocator>
using TSafeMap = TMap<KeyType, ValueType, TInstrumentationSafeSetAllocator<BaseAllocator>>;

template <typename ElementType, typename BaseAllocator = TInstrumentationSafeBaseAllocator>
using TSafeSet = TSet<ElementType, DefaultKeyFuncs<ElementType>, TInstrumentationSafeSetAllocator<BaseAllocator>>;

#endif