// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// This header is used for Core delegates that we don't want to expose outside of the engine

#include "Delegates/Delegate.h"
#include "Misc/AES.h"
#include "Misc/Guid.h"

class IPakFile;

enum class EMountOperation
{
	Mount,
	Unmount
};

/** Delegate to indicate that a pakfile has been mounted/unmounted */
DECLARE_TS_MULTICAST_DELEGATE_ThreeParams(FMountOperationPak, EMountOperation /*Operation*/, const TCHAR* /*PakPath*/, int32 /*Order*/);

struct FMountedPakInfo
{
	FMountedPakInfo() = default;
	FMountedPakInfo(const IPakFile* InPakFile, int32 InOrder)
		: PakFile(InPakFile)
		, Order(InOrder)
	{}

	~FMountedPakInfo() = default;

	const IPakFile* PakFile = nullptr;
	int32 Order = INDEX_NONE;
};

using FCurrentlyMountedPaksDelegate = TDelegate<TArray<FMountedPakInfo>()>;

struct FCoreInternalDelegates
{
	// Callback to prompt the pak system to unmount a pak file.
	static CORE_API FMountOperationPak& GetOnPakMountOperation();

	static CORE_API FCurrentlyMountedPaksDelegate& GetCurrentlyMountedPaksDelegate();
};
