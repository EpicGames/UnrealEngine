// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Mutex.h"
#include "Async/UniqueLock.h"
#include "Containers/Map.h"
#include "Delegates/Delegate.h"
#include "Misc/AES.h"
#include "Misc/Guid.h"

#define UE_API CORE_API

namespace UE
{

/** Manages a set of registered encryption key(s). */
class FEncryptionKeyManager
{
public:
	FEncryptionKeyManager(const FEncryptionKeyManager&) = delete;
	FEncryptionKeyManager(FEncryptionKeyManager&&) = delete;
	FEncryptionKeyManager& operator=(const FEncryptionKeyManager&) = delete;
	FEncryptionKeyManager& operator=(FEncryptionKeyManager&&) = delete;
	UE_API ~FEncryptionKeyManager();

	/** Returns whether the specified encrypton key exist or not. */
	UE_API bool ContainsKey(const FGuid& Id) const;
	/** Add a new encryption key, ignored if the key already exist. */
	UE_API void AddKey(const FGuid& Id, const FAES::FAESKey& Key);
	/** Try retrieve the encryption key for the specified key ID. */
	UE_API bool TryGetKey(const FGuid& Id, FAES::FAESKey& OutKey) const;
	/** Returns a map of all available keys */
	UE_API TMap<FGuid, FAES::FAESKey> GetAllKeys() const;

	DECLARE_MULTICAST_DELEGATE_TwoParams(FEncryptionKeyAddedDelegate, const FGuid&, const FAES::FAESKey&);
	/** Event triggered when a new key as been added. */
	FEncryptionKeyAddedDelegate& OnKeyAdded() { return KeyAdded; }
	/** Returns the single instance of the key manager. */
	static UE_API FEncryptionKeyManager& Get();

private:
	UE_API FEncryptionKeyManager();

	mutable FMutex Mutex;
	TMap<FGuid, FAES::FAESKey> Keys;
	FEncryptionKeyAddedDelegate KeyAdded;
};

} // namespace UE

#undef UE_API
