// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

namespace UE::Core::Private
{

class FPlayInEditorLoadingScope
{
public:
	CORE_API FPlayInEditorLoadingScope(int32 PlayInEditorID);
	CORE_API ~FPlayInEditorLoadingScope();
private:
	int32 OldValue;
};

}