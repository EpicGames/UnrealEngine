// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/CoreMiscDefines.h"

#if USING_INSTRUMENTATION

#include "CoreTypes.h"
#include "Instrumentation/Defines.h"
#include "Sanitizer/RaceDetectorTypes.h"

DECLARE_LOG_CATEGORY_EXTERN(LogRaceDetector, Log, All);

namespace UE::Sanitizer::RaceDetector {

	// Returns whether we should instrument depending on the current context state.
	INSTRUMENTATION_FUNCTION_ATTRIBUTES bool ShouldInstrument(FContext& Context);
	// Gets the current thread context, could be nullptr.
	INSTRUMENTATION_FUNCTION_ATTRIBUTES FContext* GetThreadContext();
	// Hints the sanitizer that this memory range is being freed.
	INSTRUMENTATION_FUNCTION_ATTRIBUTES void FreeMemoryRange(void* Ptr, uint64 Size);

	// Makes sure the current thread has a context and returns it.
	INSTRUMENTATION_FUNCTION_ATTRIBUTES FContext& EnsureCurrentContext();
	// Releases the current thread context.
	INSTRUMENTATION_FUNCTION_ATTRIBUTES void ReleaseCurrentContext();
	// Returns a sync object for the given address, initialize one if there isn't one already.
	INSTRUMENTATION_FUNCTION_ATTRIBUTES FSyncObjectRef GetSyncObject(FContext& Context, void* SyncAddr);
}

#endif // USING_INSTRUMENTATION