// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/CoreMiscDefines.h"

#if USING_INSTRUMENTATION

#include "CoreTypes.h"
#include "Instrumentation/Defines.h"

namespace UE::Sanitizer::RaceDetector {
	namespace Platform {
		// Performs any platform specific initialization.
		INSTRUMENTATION_FUNCTION_ATTRIBUTES bool  InitializePlatform();
		// Performs any platform specific cleanup.
		INSTRUMENTATION_FUNCTION_ATTRIBUTES bool  CleanupPlatform();
		// Prepare address space of shadow memory
		INSTRUMENTATION_FUNCTION_ATTRIBUTES void  InitShadowMemory();
		// Returns the command line the process was started with.
		INSTRUMENTATION_FUNCTION_ATTRIBUTES const TCHAR* GetCommandLine();
		// Returns the page size granularity.
		INSTRUMENTATION_FUNCTION_ATTRIBUTES UPTRINT GetPageSize();
		// Returns the base address of the shadow memory address space.
		// Note: InitShadowMemory must have been called before, otherwise this is meaningless.
		INSTRUMENTATION_FUNCTION_ATTRIBUTES UPTRINT GetShadowMemoryBase();
		// Returns the size of the entire shadow memory address space.
		// Note: InitShadowMemory must have been called before, otherwise this is meaningless.
		INSTRUMENTATION_FUNCTION_ATTRIBUTES UPTRINT GetShadowMemorySize();
		// Get the base of the shadow memory for clock banks.
		INSTRUMENTATION_FUNCTION_ATTRIBUTES UPTRINT GetShadowClockBase();
		// Returns the number of bytes currently mapped in the shadow memory.
		INSTRUMENTATION_FUNCTION_ATTRIBUTES uint64 GetShadowMemoryUsage();
		// Returns whether a particular range is already accessible in shadow memory.
		INSTRUMENTATION_FUNCTION_ATTRIBUTES bool IsShadowMemoryMapped(UPTRINT Base, UPTRINT Size);
		// Maps a range in shadow memory so that it is safe to access.
		INSTRUMENTATION_FUNCTION_ATTRIBUTES void MapShadowMemory(UPTRINT Base, UPTRINT Size);
		// Unmaps the entire range of shadow memory.
		INSTRUMENTATION_FUNCTION_ATTRIBUTES void UnmapShadowMemory();
		// Returns whether a debugger is currently attached to our process.
		INSTRUMENTATION_FUNCTION_ATTRIBUTES bool IsDebuggerPresent();
		// Returns whether there is any page currently mapped in shadow memory.
		INSTRUMENTATION_FUNCTION_ATTRIBUTES bool HasShadowMemoryMapped();
		// Sends a hint to visual studio to hide first chance exception to reduce noise caused by shadow memory access.
		INSTRUMENTATION_FUNCTION_ATTRIBUTES void HideFirstChanceExceptionInVisualStudio();
		// Sleeps for the given amount of milliseconds.
		INSTRUMENTATION_FUNCTION_ATTRIBUTES void SleepMS(uint32 Milliseconds);
		// Capture the current callstack.
		INSTRUMENTATION_FUNCTION_ATTRIBUTES uint16 CaptureStackBackTrace(uint32 FrameToSkip, uint32 FrameToCapture, void** Backtrace);
		// Allocates a Tls index.
		INSTRUMENTATION_FUNCTION_ATTRIBUTES uint32 AllocTlsSlot();
		// Releases a Tls index.
		INSTRUMENTATION_FUNCTION_ATTRIBUTES void FreeTlsSlot(uint32 Index);
		// Gets the value of the Tls index for the current thread.
		INSTRUMENTATION_FUNCTION_ATTRIBUTES void* GetTlsValue(uint32 Index);
		// Sets the value of the Tls index for the current thread.
		INSTRUMENTATION_FUNCTION_ATTRIBUTES void SetTlsValue(uint32 Index, void* Value);
		// Returns the current thread Id.
		INSTRUMENTATION_FUNCTION_ATTRIBUTES uint32 GetCurrentThreadId();

		// Rewrites the patchable function prefix with the proper jmp to the target.
		INSTRUMENTATION_FUNCTION_ATTRIBUTES void PrepareTrampoline(void* PatchableFunctionAddress, void* TargetFunctionAddress, bool bUseRETBypass);
		// Rewrites the patchable function prefix back to its original compiled NOPs.
		INSTRUMENTATION_FUNCTION_ATTRIBUTES void CleanupTrampoline(void* PatchableFunctionAddress);
		// Rewrites the 2 first bytes of the function to jump at the beginning of the prefix section.
		INSTRUMENTATION_FUNCTION_ATTRIBUTES void ActivateTrampoline(void* PatchableFunctionAddress);
		// Rewrites the 2 first bytes of the function to either do nothing or do an immediate return (RET bypass).
		INSTRUMENTATION_FUNCTION_ATTRIBUTES void DeactivateTrampoline(void* PatchableFunctionAddress, bool bUseRETBypass);
		// This needs to be called after trampoline activation or deactivation to make sure it takes effect immediately.
		INSTRUMENTATION_FUNCTION_ATTRIBUTES void FlushInstructionCache();
		// Get the limits of the stack for the current thread.
		INSTRUMENTATION_FUNCTION_ATTRIBUTES void GetCurrentThreadStackLimits(void** LowLimit, void** HighLimit);
		// Check if the thread id given is currently alive
		INSTRUMENTATION_FUNCTION_ATTRIBUTES bool IsThreadAlive(uint32 ThreadId);
		// Provides a compilation barrier
		INSTRUMENTATION_FUNCTION_ATTRIBUTES FORCEINLINE void AsymmetricThreadFenceLight()
		{
			_ReadWriteBarrier();
		}

		// The function generates an interprocessor interrupt (IPI) to all processors that are part of the current process affinity.
		// It guarantees the visibility of write operations performed on one processor to the other processors.
		INSTRUMENTATION_FUNCTION_ATTRIBUTES void AsymmetricThreadFenceHeavy();

		// Linear allocator for sync objects — reserves virtual address space, commits in chunks,
		// decommits the entire region in a single syscall on shadow reset.
		INSTRUMENTATION_FUNCTION_ATTRIBUTES void  InitSyncObjectAllocator(SIZE_T ReservedSize, SIZE_T CommitGranularity);
		INSTRUMENTATION_FUNCTION_ATTRIBUTES void  ShutdownSyncObjectAllocator();
		INSTRUMENTATION_FUNCTION_ATTRIBUTES void* SyncObjectAllocatorAlloc(SIZE_T Size);
		INSTRUMENTATION_FUNCTION_ATTRIBUTES void  SyncObjectAllocatorReset();
		INSTRUMENTATION_FUNCTION_ATTRIBUTES void  SyncObjectAllocatorAddRef();
		INSTRUMENTATION_FUNCTION_ATTRIBUTES void  SyncObjectAllocatorRelease();
		INSTRUMENTATION_FUNCTION_ATTRIBUTES int32 SyncObjectAllocatorGetRefCount();
		INSTRUMENTATION_FUNCTION_ATTRIBUTES SIZE_T SyncObjectAllocatorGetCommittedSize();
	}
}

#endif // USING_INSTRUMENTATION