// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/CoreMiscDefines.h"

#if USING_INSTRUMENTATION

#include "CoreTypes.h"

#include "Sanitizer/Types.h"
#include "Sanitizer/RaceDetectorPlatform.h"
#include "Instrumentation/Types.h"
#include "Instrumentation/Containers.h"

extern int32 GRaceDetectorHistoryLength;

namespace UE::Sanitizer::RaceDetector {

	using namespace UE::Instrumentation;

	extern volatile bool bDetailedLogGlobal;

	// A read-write lock that doesn't put the thread into a WAIT state but instead repeatedly tries to acquire the lock.
	// This version is customized to remove instrumentation and be as optimized as possible for instrumentation purpose.
	// FPlatformAtomics are used to make sure all atomics are inlined.
	// std::atomic often end up causing calls into non-inlined instrumented functions that causes costly reentrancy.
	class FRWSpinLock
	{
	public:
		UE_NONCOPYABLE(FRWSpinLock);

		INSTRUMENTATION_FUNCTION_ATTRIBUTES FORCEINLINE FRWSpinLock() = default;

		INSTRUMENTATION_FUNCTION_ATTRIBUTES FORCEINLINE bool TryWriteLock()
		{
			return FPlatformAtomics::InterlockedCompareExchange((volatile int32*)&Lock, UINT32_MAX, 0) == 0;
		}

		INSTRUMENTATION_FUNCTION_ATTRIBUTES FORCEINLINE void WriteLock()
		{
			while (!TryWriteLock())
			{
				// Reduce contention by doing a simple relaxed read to see if we have a chance of being able to lock.
				while (Lock != 0)
				{
					FPlatformProcess::Yield();
				}
			}
		}

		INSTRUMENTATION_FUNCTION_ATTRIBUTES FORCEINLINE void WriteUnlock()
		{
			Lock = 0;
		}

		INSTRUMENTATION_FUNCTION_ATTRIBUTES FORCEINLINE bool TryReadLock()
		{
			uint32 LocalValue = Lock;
			// Check to make sure we don't already have a write lock or that we've not reached the limit of reader locks.
			if (LocalValue >= UINT32_MAX - 1)
			{
				return false;
			}

			return FPlatformAtomics::InterlockedCompareExchange((volatile int32*)&Lock, LocalValue + 1, LocalValue) == LocalValue;
		}

		INSTRUMENTATION_FUNCTION_ATTRIBUTES FORCEINLINE void ReadUnlock()
		{
			FPlatformAtomics::InterlockedDecrement((volatile int32*)&Lock);
		}

		INSTRUMENTATION_FUNCTION_ATTRIBUTES FORCEINLINE void ReadLock()
		{
			while (!TryReadLock())
			{
				FPlatformProcess::Yield();
			}
		}

	private:
		volatile uint32 Lock = 0;
	};

	template<typename MutexType>
	class TReadScopeLock
	{
	public:
		UE_NONCOPYABLE(TReadScopeLock);

		INSTRUMENTATION_FUNCTION_ATTRIBUTES FORCEINLINE TReadScopeLock(MutexType& InMutex)
			: Mutex(InMutex)
		{
			Mutex.ReadLock();
		}

		INSTRUMENTATION_FUNCTION_ATTRIBUTES FORCEINLINE ~TReadScopeLock()
		{
			Mutex.ReadUnlock();
		}

	private:
		MutexType& Mutex;
	};

	template<typename MutexType>
	class TWriteScopeLock
	{
	public:
		UE_NONCOPYABLE(TWriteScopeLock);

		INSTRUMENTATION_FUNCTION_ATTRIBUTES FORCEINLINE TWriteScopeLock(MutexType& InMutex)
			: Mutex(InMutex)
		{
			Mutex.WriteLock();
		}

		INSTRUMENTATION_FUNCTION_ATTRIBUTES FORCEINLINE ~TWriteScopeLock()
		{
			Mutex.WriteUnlock();
		}

	private:
		MutexType& Mutex;
	};

	enum class ERWScopeLockType
	{
		SLT_ReadOnly = 0,
		SLT_Write,
	};

	template<typename MutexType>
	class TRWScopeLock
	{
	public:
		UE_NONCOPYABLE(TRWScopeLock);

		INSTRUMENTATION_FUNCTION_ATTRIBUTES FORCEINLINE TRWScopeLock(MutexType& InMutex, ERWScopeLockType InLockType)
			: Mutex(InMutex)
			, LockType(InLockType)
		{
			if (LockType == ERWScopeLockType::SLT_ReadOnly)
			{
				Mutex.ReadLock();
			}
			else
			{
				Mutex.WriteLock();
			}
		}

		INSTRUMENTATION_FUNCTION_ATTRIBUTES FORCEINLINE ~TRWScopeLock()
		{
			if (LockType == ERWScopeLockType::SLT_ReadOnly)
			{
				Mutex.ReadUnlock();
			}
			else
			{
				Mutex.WriteUnlock();
			}
		}

	private:
		MutexType& Mutex;
		ERWScopeLockType LockType;
	};

	// ------------------------------------------------------------------------------
	// Clocks.
	// ------------------------------------------------------------------------------
	using FClock = uint32;
	using FContextId = uint8;

	class FClockBank 
	{
	public:
		INSTRUMENTATION_FUNCTION_ATTRIBUTES FClockBank()
		{
			Reset();
		}

		INSTRUMENTATION_FUNCTION_ATTRIBUTES void Acquire(FClockBank& Other, void* ReturnAddress)
		{
			for (int32 Index = 0; Index < UE_ARRAY_COUNT(Clocks); ++Index)
			{
				if (Other.Clocks[Index] > Clocks[Index])
				{
					Clocks[Index] = Other.Clocks[Index];
					Locations[Index] = ReturnAddress;
				}
			}
		}

		INSTRUMENTATION_FUNCTION_ATTRIBUTES void Release(FClockBank& Other, void* ReturnAddress)
		{
			Other.Acquire(*this, ReturnAddress);
		}

		INSTRUMENTATION_FUNCTION_ATTRIBUTES void AcquireRelease(FClockBank& Other, void* ReturnAddress)
		{
			for (int32 Index = 0; Index < UE_ARRAY_COUNT(Clocks); ++Index)
			{
				if (Clocks[Index] > Other.Clocks[Index])
				{
					Other.Clocks[Index] = Clocks[Index];
					Other.Locations[Index] = ReturnAddress;
				}
				else if (Other.Clocks[Index] > Clocks[Index])
				{
					Clocks[Index] = Other.Clocks[Index];
					Locations[Index] = ReturnAddress;
				}
			}
		}

		INSTRUMENTATION_FUNCTION_ATTRIBUTES void Reset()
		{
			memset(Clocks, 0, sizeof(Clocks));
			memset(Locations, 0, sizeof(Locations));
		}

		INSTRUMENTATION_FUNCTION_ATTRIBUTES FClock& Get(FContextId ContextId)
		{
			return Clocks[ContextId];
		}

		INSTRUMENTATION_FUNCTION_ATTRIBUTES FCallstackLocation GetLocation(FContextId ContextId)
		{
			return FCallstackLocation(&Locations[ContextId], 1);
		}

	private:
		FClock Clocks[256] = {};
		void* Locations[256] = {};
	};

	// ------------------------------------------------------------------------------
	// Memory access.
	// ------------------------------------------------------------------------------
	enum EMemoryAccessType : uint8 {
		ACCESS_TYPE_INVALID = 0b0,
		ACCESS_TYPE_READ = 0b1,
		ACCESS_TYPE_WRITE = 0b10,
		ACCESS_TYPE_ATOMIC = 0b100,
		ACCESS_TYPE_VPTR = 0b1000,
		ACCESS_TYPE_ALWAYSREPORT = 0b10000,

		ACCESS_TYPE_ATOMIC_READ = ACCESS_TYPE_ATOMIC | ACCESS_TYPE_READ,
		ACCESS_TYPE_ATOMIC_WRITE = ACCESS_TYPE_ATOMIC | ACCESS_TYPE_WRITE,
		ACCESS_TYPE_ATOMIC_READ_WRITE = ACCESS_TYPE_ATOMIC_READ | ACCESS_TYPE_ATOMIC_WRITE
	};
	INSTRUMENTATION_ENUM_CLASS_FLAGS(EMemoryAccessType);

	INSTRUMENTATION_FUNCTION_ATTRIBUTES inline const TCHAR* AccessTypeToString(EMemoryAccessType AccessType)
	{
		switch (AccessType & ACCESS_TYPE_ATOMIC_READ_WRITE)
		{
		case ACCESS_TYPE_READ:
			return TEXT("Read");
		case ACCESS_TYPE_WRITE:
			return TEXT("Write");
		case ACCESS_TYPE_ATOMIC_READ:
			return TEXT("AtomicRead");
		case ACCESS_TYPE_ATOMIC_WRITE:
			return TEXT("AtomicWrite");
		case ACCESS_TYPE_ATOMIC_READ_WRITE:
			return TEXT("AtomicReadWrite");
		}
		return TEXT("Unknown");
	}
	INSTRUMENTATION_FUNCTION_ATTRIBUTES inline bool IsReadMemoryAccess(EMemoryAccessType AccessType)
	{
		return EnumHasAnyFlags(AccessType, EMemoryAccessType::ACCESS_TYPE_READ);
	}
	INSTRUMENTATION_FUNCTION_ATTRIBUTES inline bool IsWriteMemoryAccess(EMemoryAccessType AccessType)
	{
		return EnumHasAnyFlags(AccessType, EMemoryAccessType::ACCESS_TYPE_WRITE);
	}
	INSTRUMENTATION_FUNCTION_ATTRIBUTES inline bool IsAtomicMemoryAccess(EMemoryAccessType AccessType)
	{
		return EnumHasAnyFlags(AccessType, EMemoryAccessType::ACCESS_TYPE_ATOMIC);
	}

	struct FMemoryAccess 
	{
		INSTRUMENTATION_FUNCTION_ATTRIBUTES FORCEINLINE FMemoryAccess()
		{
		}

		// Not sure why yet but Clang is not inlining this by default
		// Did we mess up the clang compilation settings??
		INSTRUMENTATION_FUNCTION_ATTRIBUTES FORCEINLINE FMemoryAccess(uint64 InRawValue)
			: RawValue(InRawValue)
		{
		}

		INSTRUMENTATION_FUNCTION_ATTRIBUTES FORCEINLINE FMemoryAccess(FContextId InContextId, FClock InClock, uint8 InOffset, uint8 InSize, EMemoryAccessType InAccessType, bool bInAlwaysReport)
		{
			// Apparently this is much faster than using the bitfields. 30% faster in warm TSAN benchmark!!!
			// The constructor was accessing the same value in memory/store buffer multiple time doing its bit tweaking and it caused tons of Core::X86::Pmc::Core::LsBadStatus2.
			// That ended up stalling on S[0].RawValue == CurrentAccess.RawValue in InstrumentMemoryAccess when trying to extract the full uint64.
			// https://blog.stuffedcow.net/2014/01/x86-memory-disambiguation/
			const uint8 AccessValue = (uint8)(((1ull << InSize) - 1ull) << InOffset);
			if (bInAlwaysReport)
			{
				InAccessType |= EMemoryAccessType::ACCESS_TYPE_ALWAYSREPORT;
			}
			RawValue = (uint64)InAccessType << 48 | (uint64)AccessValue << 40 | (uint64)InContextId << 32 | (uint64)InClock;
		}
		
		INSTRUMENTATION_FUNCTION_ATTRIBUTES FORCEINLINE uint8 GetOffset() const { return (uint8)FMath::CountTrailingZeros(Access); }
		INSTRUMENTATION_FUNCTION_ATTRIBUTES FORCEINLINE uint8 GetSize() const { return (uint8)FMath::CountBits(Access >> GetOffset()); }
		INSTRUMENTATION_FUNCTION_ATTRIBUTES FORCEINLINE bool IsValid() const { return AccessType != EMemoryAccessType::ACCESS_TYPE_INVALID; }

		union
		{
			// If you touch this layout, make sure to update the constructor above
			struct
			{
				FClock     Clock;                    // 0
				FContextId ContextId;                // 4
				// Each bit represent a 1-byte slot used in our 8 byte shadow
				// and can easily be tested for overlaps with other accesses.
				uint8      Access;                   // 5
				union
				{
					struct
					{
						uint8 bIsRead       : 1;
						uint8 bIsWrite      : 1;
						uint8 bIsAtomic     : 1;
						uint8 bIsVPtr       : 1;
						uint8 bAlwaysReport : 1;
					};
					EMemoryAccessType AccessType : 5; // 6  plenty of bits left here
				};
				uint8 Reserved;                           // 7  plenty of bits left here
			};

			uint64 RawValue;
		};
	};

	enum class EHistoryEntryType : uint8
	{
		Invalid       = 0,     // Just make sure we do not mistake 0 memory for something valid
		FunctionEntry = 0xAA,  // Any number will do but make them stand out in the trace
		MemoryAccess  = 0xBB,
		FunctionExit  = 0xCC
	};

	struct FHistoryEntryBase
	{
		EHistoryEntryType  Type;

		INSTRUMENTATION_FUNCTION_ATTRIBUTES FHistoryEntryBase(EHistoryEntryType InType)
			: Type(InType)
		{
		}
	};

	struct FHistoryEntryAccess : public FHistoryEntryBase
	{
		void*          Pointer;
		FMemoryAccess  Access;

		INSTRUMENTATION_FUNCTION_ATTRIBUTES FHistoryEntryAccess(void* InPointer, uint64 InRawAccess)
			: FHistoryEntryBase(EHistoryEntryType::MemoryAccess)
			, Pointer(InPointer)
			, Access(InRawAccess)
		{
		}
	};

	struct FHistoryEntryFunctionEntry : public FHistoryEntryBase
	{
		void* ReturnAddress;

		INSTRUMENTATION_FUNCTION_ATTRIBUTES FHistoryEntryFunctionEntry(void* InReturnAddress)
			: FHistoryEntryBase(EHistoryEntryType::FunctionEntry)
			, ReturnAddress(InReturnAddress)
		{
		}
	};

	struct FHistoryEntryFunctionExit : public FHistoryEntryBase
	{
		INSTRUMENTATION_FUNCTION_ATTRIBUTES FHistoryEntryFunctionExit()
			: FHistoryEntryBase(EHistoryEntryType::FunctionExit)
		{
		}
	};

	struct FHistoryChunk
	{
		SAFE_OPERATOR_NEW_DELETE()

		INSTRUMENTATION_FUNCTION_ATTRIBUTES FHistoryChunk();
		INSTRUMENTATION_FUNCTION_ATTRIBUTES ~FHistoryChunk();
		INSTRUMENTATION_FUNCTION_ATTRIBUTES void InitStack();

		uint32 StartClock = 0;
		uint32 EndClock = 0;
		uint32 Offset = 0;
		double LastUsed = FPlatformTime::Seconds();
		uint8  Buffer[2*1024*1024];
		FHistoryChunk* Prev = nullptr;
		FHistoryChunk* Next = nullptr;
	};

	struct FClockRange
	{
		FClock First;
		FClock Last;
	};

	struct FAccessHistory
	{
		SAFE_OPERATOR_NEW_DELETE()

		INSTRUMENTATION_FUNCTION_ATTRIBUTES FAccessHistory()
		{
			Tail = Head = new FHistoryChunk();
			Tail->InitStack();
			NumChunks++;
		}

		INSTRUMENTATION_FUNCTION_ATTRIBUTES ~FAccessHistory()
		{
			TWriteScopeLock Scope(Lock);

			while (Head)
			{
				FHistoryChunk* ToDelete = Head;
				Head = Head->Next;
				delete ToDelete;
			}

			NumChunks = 0;
			Head = Tail = nullptr;

			while (Spare)
			{
				FHistoryChunk* Next = Spare->Next;
				delete Spare;
				Spare = Next;
			}
		}

		// Number of chunks that have been recycled
		uint64 RecycleCount = 0;

		// Total number of chunks currently allocated
		int32  NumChunks = 0;
		int32  NumSpares = 0;

		// Used to dump information in case we can't find the memory access in the history
		double LastRecycle = 0.0;

		// Just used on the slow path between recycling and scanning.
		FRWSpinLock Lock;

		// Can be used by other threads doing race reporting
		FHistoryChunk* Head = nullptr;

		// Only used by the owner thread
		FHistoryChunk* Tail = nullptr;

		// Only used by the owner thread to store unused buffers
		FHistoryChunk* Spare = nullptr;

		INSTRUMENTATION_FUNCTION_ATTRIBUTES bool HasTooManyChunks() const
		{
			// We need to have at least 2 chunks so that we always have one filled with data
			// while we start filling the new one.
			return NumChunks > 2 && NumChunks > GRaceDetectorHistoryLength;
		}

		INSTRUMENTATION_FUNCTION_ATTRIBUTES void TrimChunks()
		{
			if (HasTooManyChunks())
			{
				TWriteScopeLock Scope(Lock);

				while (HasTooManyChunks())
				{
					FHistoryChunk* Recycle = Head;
					Head = Head->Next;
					Head->Prev = nullptr;

					RecycleCount++;
					LastRecycle = Recycle->LastUsed;

					Recycle->Next = Spare;
					Spare = Recycle;

					Spare->StartClock = 0;
					Spare->EndClock = 0;
					Spare->Offset = 0;
					Spare->LastUsed = 0.0;

					NumChunks--;
					NumSpares++;
				}
			}
		}

		INSTRUMENTATION_FUNCTION_ATTRIBUTES FORCENOINLINE void EnsureNewChunk()
		{
			FHistoryChunk* NewChunk = nullptr;
			if (Spare)
			{
				NumSpares--;
				NewChunk = Spare;
				Spare = Spare->Next;
				NewChunk->Next = nullptr;
			}
			else
			{
				NewChunk = new FHistoryChunk();
			}

			NewChunk->InitStack();

			Tail->LastUsed = FPlatformTime::Seconds();
			NewChunk->Prev = Tail;
			Tail->Next = NewChunk;
			Tail = NewChunk;
			NumChunks++;

			TrimChunks();
		}

		template <typename EntryType, typename... ArgsType>
		INSTRUMENTATION_FUNCTION_ATTRIBUTES FORCEINLINE void EmplaceEntry(ArgsType&&... Args)
		{
			if (UNLIKELY(Tail->Offset + sizeof(EntryType) > UE_ARRAY_COUNT(Tail->Buffer)))
			{
				EnsureNewChunk();
			}

			new (Tail->Buffer + Tail->Offset) EntryType(Forward<ArgsType>(Args)...);

			Platform::AsymmetricThreadFenceLight();
			Tail->Offset += sizeof(EntryType);
		}

		INSTRUMENTATION_FUNCTION_ATTRIBUTES FORCEINLINE void AddFunctionEntry(void* InReturnAddress)
		{
			EmplaceEntry<FHistoryEntryFunctionEntry>(InReturnAddress);
		}

		INSTRUMENTATION_FUNCTION_ATTRIBUTES FORCEINLINE void AddMemoryAccess(void* InPointer, const FMemoryAccess& InAccess)
		{
			EmplaceEntry<FHistoryEntryAccess>(InPointer, InAccess.RawValue);
			Tail->EndClock = InAccess.Clock;
		}

		INSTRUMENTATION_FUNCTION_ATTRIBUTES FORCEINLINE void AddFunctionExit()
		{
			EmplaceEntry<FHistoryEntryFunctionExit>();
		}

		INSTRUMENTATION_FUNCTION_ATTRIBUTES inline bool ResolveAccess(struct FContext& Context, const void* InAlignedPointer, FMemoryAccess InAccess, FCallstackLocation& OutLocation, FClockRange& OutClockRange);

		INSTRUMENTATION_FUNCTION_ATTRIBUTES int32 GetOffset() const
		{
			return Tail->Offset;
		}
	};

	struct FSyncObjectBank;
	// ------------------------------------------------------------------------------
	// Race Detector Context.
	// ------------------------------------------------------------------------------
	
	// We use ref-counting because this otherwise might get deleted
	// by other threads and we'd need to hold a lock while scanning
	// the history for race report, which would be unnaceptable.
	struct FContext : public TRefCountingMixin<FContext> {
		SAFE_OPERATOR_NEW_DELETE_WITH_GUARDS()

		INSTRUMENTATION_FUNCTION_ATTRIBUTES FContext(uint32 InThreadId)
			: ThreadId(InThreadId)
		{
		}

		INSTRUMENTATION_FUNCTION_ATTRIBUTES ~FContext()
		{
			if (AccessHistory)
			{
				delete AccessHistory;
				AccessHistory = nullptr;
			}
			if (ResolvingStack)
			{
				FInstrumentationSafeWinAllocator::FreeWithGuards(ResolvingStack);
				ResolvingStack = nullptr;
			}
		}

		uint32 GlobalEpoch = 0;
		// When we activate tracing, we need to recapture the current stack
		uint32 StackEpoch = 0;
		// Unassigned until the first memory access
		FContextId ContextId = 0;
		// Avoid reading another TLS value for this
		uint32 ThreadId;
		// Prevents recursion for instrumentation
		uint32 InstrumentationDepth = 0;
		// Prevents recursion for detoured intrumentation
		uint32 WinInstrumentationDepth = 0;
		// Used to avoid instrumenting CreateThread while inside a higher level thread creation function (i.e. beginthreadx)
		uint32 ThreadCreationDepth = 0; 
		// When we want detailed logging for diagnostic purpose
		uint32 DetailedLogDepth = 0;
		// Clock used while waiting to get a context id assigned.
		FClock StandbyClock;
		// Each thread holds a bank of clocks to synchronize with every other context.
		FClockBank ClockBank;
		// Hazard pointer used between GetSyncObject and ResetShadow
		FSyncObjectBank* BankHazard = nullptr;
		// We need to keep the callstack for each thread 
		uint16 CurrentCallstackSize = 0;
		// We use this to pass thread arguments to functions that dont have parameters (i.e. ExitThread).
		void* ThreadArgs = nullptr;
		// Avoid using UniquePtr because it's instrumented and each access has a cost.
		FAccessHistory* AccessHistory = nullptr;
		// Wether or not to always report race for this thread.
		bool bAlwaysReport = false;
		// Used when we need to resolve a stack
		void** ResolvingStack = nullptr;
		// This can be bumped again if we ever face a need for deeper callstacks since
		// this is a virtual allocation anyway so it's not going to take physical memory
		// until it is used. 
		// This needs to be at the end of the allocated block as we rely on page fault
		// to abort the program if the stack ever goes beyond this limit.
		static constexpr SIZE_T MaxCallstackSize = 4096;
		void* CurrentCallstack[MaxCallstackSize];
			
		static_assert(
			sizeof(FHistoryChunk::Buffer) > 10 * MaxCallstackSize * sizeof(FHistoryEntryFunctionEntry), 
			"FHistoryChunk::Buffer should be big enough to accomodate initial callstack with plenty of space left"
		);

		// [NO ACCESS GUARD PAGE]

		// The clock for this context Id
		INSTRUMENTATION_FUNCTION_ATTRIBUTES FClock& CurrentClock()
		{ 
			// Make sure we use the same value for the comparison and the get in case
			// this is called from another thread while we're releasing our context id.
			const FContextId LocalContextId = ContextId;
			return LocalContextId == 0 ? StandbyClock : ClockBank.Get(LocalContextId);
		}
		
		INSTRUMENTATION_FUNCTION_ATTRIBUTES void IncrementClock()
		{
			CurrentClock()++;
			
			if (UNLIKELY(DetailedLogDepth || bDetailedLogGlobal))
			{
				FPlatformMisc::LowLevelOutputDebugStringf(TEXT("[%d] Thread is now at clock %u\n"), ThreadId, CurrentClock());
			}
		}

		INSTRUMENTATION_FUNCTION_ATTRIBUTES static bool IsValid(FContext* Context)
		{
			return reinterpret_cast<int64>(Context) > 0;
		}
	};

	INSTRUMENTATION_FUNCTION_ATTRIBUTES bool FAccessHistory::ResolveAccess(FContext& Context, const void* InAlignedPointer, FMemoryAccess InAccess, FCallstackLocation& OutLocation, FClockRange& OutClockRange)
	{
		TReadScopeLock Scope(Lock);

		// Make sure we sync with the light fence.
		Platform::AsymmetricThreadFenceHeavy();

		OutClockRange.First = Head ? Head->StartClock : 0;
		OutClockRange.Last = OutClockRange.First;

		for (FHistoryChunk* Chunk = Head; Chunk; Chunk = Chunk->Next)
		{
			OutClockRange.Last = Chunk->EndClock;

			// Do not bother searching a chunk that is outside the range we're looking for.
			// Do the range comparison in a way that handles clock wrapping
			if ((InAccess.Clock - Chunk->StartClock) <= (Chunk->EndClock - Chunk->StartClock))
			{
				int32 StackNum = 0;
				if (Context.ResolvingStack == nullptr)
				{
					// Each thread has its own resolving stack with page guards to automatically protect
					// against underflow and overflow without additional logic.
					Context.ResolvingStack = (void**)FInstrumentationSafeWinAllocator::AllocWithGuards(256 * 1024);
				}

				int32 Offset = 0;
				while (Offset < Chunk->Offset)
				{
					FHistoryEntryBase* Entry = (FHistoryEntryBase*)(Chunk->Buffer + Offset);
					switch (Entry->Type)
					{
					case EHistoryEntryType::FunctionEntry:
						Context.ResolvingStack[StackNum++] = ((FHistoryEntryFunctionEntry*)Entry)->ReturnAddress;
						Offset += sizeof(FHistoryEntryFunctionEntry);
					break;
					case EHistoryEntryType::MemoryAccess:
					{
						FHistoryEntryAccess* MemoryAccessEntry = (FHistoryEntryAccess*)Entry;
						if (MemoryAccessEntry->Pointer == InAlignedPointer &&
							MemoryAccessEntry->Access.RawValue == InAccess.RawValue)
						{
							OutLocation = FCallstackLocation(Context.ResolvingStack, StackNum);
							return true;
						}
						Offset += sizeof(FHistoryEntryAccess);
					}
					break;
					case EHistoryEntryType::FunctionExit:
						StackNum--;
						Offset += sizeof(FHistoryEntryBase);
					break;
					default:
						check(false);
						// This should never happen, but if it does it is most likely a race condition
						// so just restart the tracing from the beginning as a last resort.
						StackNum = 0;
						Offset = 0;
					break;
					}
				}
			}
		}

		return false;
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES CORE_API extern FContext* GetThreadContext();

	struct FInstrumentationScope
	{
		bool bNeedDecrement = false;
		INSTRUMENTATION_FUNCTION_ATTRIBUTES FInstrumentationScope();
		INSTRUMENTATION_FUNCTION_ATTRIBUTES ~FInstrumentationScope()
		{
			if (bNeedDecrement)
			{
				GetThreadContext()->WinInstrumentationDepth--;
			}
		}
	};

	class FSyncObject
	{
	public:
		INSTRUMENTATION_FUNCTION_ATTRIBUTES void* operator new(SIZE_T Size)
		{
			FPlatformAtomics::InterlockedIncrement(&ObjectCount);
			return Platform::SyncObjectAllocatorAlloc(Size);
		}

		INSTRUMENTATION_FUNCTION_ATTRIBUTES void operator delete(void* Ptr)
		{
			// No individual memory free — the linear allocator decommits everything on reset.
			FPlatformAtomics::InterlockedDecrement(&ObjectCount);
		}

		template <typename AtomicOpType>
		INSTRUMENTATION_FUNCTION_ATTRIBUTES void SyncAcquire(FContext& Context, AtomicOpType&& AtomicOp, void* ReturnAddress, void* SyncAddress, const TCHAR* OpName)
		{
			TWriteScopeLock Scope(Lock);
			SyncAcquireAsSoleOwnerOrReadOwner(Context, ReturnAddress, SyncAddress, OpName);
			AtomicOp();
		}

		template <typename AtomicOpType>
		INSTRUMENTATION_FUNCTION_ATTRIBUTES void SyncRelease(FContext& Context, AtomicOpType&& AtomicOp, void* ReturnAddress, void* SyncAddress, const TCHAR* OpName)
		{
			TWriteScopeLock Scope(Lock);
			SyncReleaseAsSoleOwner(Context, ReturnAddress, SyncAddress, OpName);
			AtomicOp();
		}

		template <typename AtomicOpType>
		INSTRUMENTATION_FUNCTION_ATTRIBUTES void SyncAcquireRelease(FContext& Context, AtomicOpType&& AtomicOp, void* ReturnAddress, void* SyncAddress, const TCHAR* OpName)
		{
			if (UNLIKELY(Context.DetailedLogDepth || bDetailedLogGlobal))
			{
				FPlatformMisc::LowLevelOutputDebugStringf(TEXT("[%d] %s acq_rel of 0x%p from function at 0x%p\n"), Context.ThreadId, OpName, SyncAddress, ReturnAddress);
			}

			TWriteScopeLock Scope(Lock);
			Context.ClockBank.AcquireRelease(ClockBank, ReturnAddress);
			AtomicOp();
		}

		template <typename AtomicOpType, typename ActualAccessCallbackType>
		INSTRUMENTATION_FUNCTION_ATTRIBUTES void SyncWithFailureSupport(FContext& Context, AtomicOpType&& AtomicOp, EMemoryAccessType AccessType, FAtomicMemoryOrder SuccessOrder, FAtomicMemoryOrder FailureOrder, void* ReturnAddress, void* SyncAddress, const TCHAR* OpName, ActualAccessCallbackType&& ActualAccessCallback)
		{
			using namespace UE::Instrumentation;

			// We only need a take a write lock when we do a release or acq_rel operation otherwise
			// it's impossible to test for failure order as the AtomicOp inside the write lock would never fail.
			// An acquire only operation is safe to run under read-lock since we're reading from the syncobject and writing into the context clockbank which is owned by the current thread.
			// Per the standard, failure memory order cannot be release nor acq_release, so we don't need to look at the failure order to choose our lock type.
			// See https://www.open-std.org/jtc1/sc22/wg21/docs/papers/2017/n4659.pdf §32.6.1
			// #17 Requires: The failure argument shall not be memory_order_release nor memory_order_acq_rel.
			// 
			// Also for the time being MSVC atomic implementation always combines both order so we have no way to test our support of different SuccessOrder and FailureOrder until they fix their implementation.
			TRWScopeLock Scope(Lock, IsAtomicOrderRelease(SuccessOrder) ? ERWScopeLockType::SLT_Write : ERWScopeLockType::SLT_ReadOnly);
			bool bSucceeded = AtomicOp();

			FAtomicMemoryOrder Order = bSucceeded ? SuccessOrder : FailureOrder;
			const TCHAR* OpResult = bSucceeded ? TEXT("success") : TEXT("failure");

			ActualAccessCallback(Order);

			if (IsAtomicOrderRelaxed(Order))
			{
				if (UNLIKELY(Context.DetailedLogDepth || bDetailedLogGlobal))
				{
					FPlatformMisc::LowLevelOutputDebugStringf(TEXT("[%d] %s %s relaxed of 0x%p from function at 0x%p\n"), Context.ThreadId, OpName, OpResult, SyncAddress, ReturnAddress);
				}

				// Do nothing in the relaxed case since no barrier is provided.
				return;
			}
			else if (AccessType == ACCESS_TYPE_ATOMIC_READ_WRITE && IsAtomicOrderAcquireRelease(Order))
			{
				if (UNLIKELY(Context.DetailedLogDepth || bDetailedLogGlobal))
				{
					FPlatformMisc::LowLevelOutputDebugStringf(TEXT("[%d] %s %s acq_rel of 0x%p from function at 0x%p\n"), Context.ThreadId, OpName, OpResult, SyncAddress, ReturnAddress);
				}

				Context.ClockBank.AcquireRelease(ClockBank, ReturnAddress);
			}
			else if ((AccessType & ACCESS_TYPE_ATOMIC_READ) == ACCESS_TYPE_ATOMIC_READ && IsAtomicOrderAcquire(Order))
			{
				if (UNLIKELY(Context.DetailedLogDepth || bDetailedLogGlobal))
				{
					FPlatformMisc::LowLevelOutputDebugStringf(TEXT("[%d] %s %s acquire 0x%p from function at 0x%p\n"), Context.ThreadId, OpName, OpResult, SyncAddress, ReturnAddress);
				}

				Context.ClockBank.Acquire(ClockBank, ReturnAddress);
			}
			else if ((AccessType & ACCESS_TYPE_ATOMIC_WRITE) == ACCESS_TYPE_ATOMIC_WRITE && IsAtomicOrderRelease(Order))
			{
				if (UNLIKELY(Context.DetailedLogDepth || bDetailedLogGlobal))
				{
					FPlatformMisc::LowLevelOutputDebugStringf(TEXT("[%d] %s %s release 0x%p from function at 0x%p\n"), Context.ThreadId, OpName, OpResult, SyncAddress, ReturnAddress);
				}

				Context.ClockBank.Release(ClockBank, ReturnAddress);
			}
			else
			{
				checkf(false, TEXT("Unexpected memory order"));
			}

			Context.IncrementClock();
		}

		// Must be called by a thread that has either this object's spin lock,
		// or an external lock that is guaranteed to be held.
		INSTRUMENTATION_FUNCTION_ATTRIBUTES void SyncReleaseAsSoleOwner(FContext& Context, void* ReturnAddress, void* SyncAddress, const TCHAR* OpName)
		{
			if (UNLIKELY(Context.DetailedLogDepth || bDetailedLogGlobal))
			{
				FPlatformMisc::LowLevelOutputDebugStringf(TEXT("[%d] %s releases 0x%p from function at 0x%p\n"), Context.ThreadId, OpName, SyncAddress, ReturnAddress);
			}

			Context.ClockBank.Release(ClockBank, ReturnAddress);
		}

		INSTRUMENTATION_FUNCTION_ATTRIBUTES void SyncAcquireAsSoleOwnerOrReadOwner(FContext& Context, void* ReturnAddress, void* SyncAddress, const TCHAR* OpName)
		{
			if (UNLIKELY(Context.DetailedLogDepth || bDetailedLogGlobal))
			{
				FPlatformMisc::LowLevelOutputDebugStringf(TEXT("[%d] %s acquires 0x%p from function at 0x%p\n"), Context.ThreadId, OpName, SyncAddress, ReturnAddress);
		 	}

			Context.ClockBank.Acquire(ClockBank, ReturnAddress);
		}

		INSTRUMENTATION_FUNCTION_ATTRIBUTES static uint64 GetObjectCount()
		{
			return ObjectCount;
		}

		INSTRUMENTATION_FUNCTION_ATTRIBUTES static void ResetObjectCount()
		{
			ObjectCount = 0;
		}

	private:
		FRWSpinLock Lock;
		FClockBank ClockBank;
		static volatile int64 ObjectCount;
	};

	// One SyncObjectBank per 64-bit aligned address.
	// Lives in the linear allocator — never individually freed.
	struct FSyncObjectBank
	{
		INSTRUMENTATION_FUNCTION_ATTRIBUTES void* operator new(SIZE_T Size)
		{
			FPlatformAtomics::InterlockedIncrement(&ObjectCount);
			return Platform::SyncObjectAllocatorAlloc(Size);
		}

		INSTRUMENTATION_FUNCTION_ATTRIBUTES void operator delete(void* Ptr)
		{
			// No individual memory free — the linear allocator decommits everything on reset.
			FPlatformAtomics::InterlockedDecrement(&ObjectCount);
		}

		INSTRUMENTATION_FUNCTION_ATTRIBUTES static uint64 GetObjectCount()
		{
			return ObjectCount;
		}

		INSTRUMENTATION_FUNCTION_ATTRIBUTES static void ResetObjectCount()
		{
			ObjectCount = 0;
		}

		INSTRUMENTATION_FUNCTION_ATTRIBUTES FSyncObject* GetSyncObject(int32 Index)
		{
			if (SyncObjects[Index] == nullptr)
			{
				FSyncObject* SyncObject = new FSyncObject();
				if (FPlatformAtomics::InterlockedCompareExchangePointer((void**)&SyncObjects[Index], SyncObject, nullptr) != nullptr)
				{
					delete SyncObject;
				}
			}

			return SyncObjects[Index];
		}

		static volatile int64 ObjectCount;
	private:
		// One sync object per byte offset in the 8-byte aligned address.
		// Allocated on demand from the linear allocator.
		FSyncObject* SyncObjects[8] = { 0 };
	};

	struct FSyncObjectRef
	{
		INSTRUMENTATION_FUNCTION_ATTRIBUTES FORCEINLINE FSyncObjectRef(FSyncObjectBank* InBank, FSyncObject* InSyncObject)
			: Object(InSyncObject)
		{
			Platform::SyncObjectAllocatorAddRef();
		}

		INSTRUMENTATION_FUNCTION_ATTRIBUTES FORCEINLINE ~FSyncObjectRef()
		{
			if (Object)
			{
				Platform::SyncObjectAllocatorRelease();
			}
		}

		FSyncObjectRef(const FSyncObjectRef&) = delete;
		FSyncObjectRef& operator=(const FSyncObjectRef&) = delete;
		FSyncObjectRef& operator=(FSyncObjectRef&&) = delete;

		INSTRUMENTATION_FUNCTION_ATTRIBUTES FSyncObjectRef(FSyncObjectRef&& Other)
			: Object(Other.Object)
		{
			Other.Object = nullptr;
		}

		INSTRUMENTATION_FUNCTION_ATTRIBUTES FORCEINLINE FSyncObject* operator->() const
		{
			return Object;
		}
	private:
		FSyncObject* Object;
	};

	// ------------------------------------------------------------------------------
	// Shadow memory.
	// ------------------------------------------------------------------------------
	struct FShadowMemory 
	{
		FMemoryAccess Accesses[4];
	};

	struct FShadowClockBankSlot
	{
		FSyncObjectBank* SyncObjectBank;
	};

} // UE::Sanitizer::RaceDetector

#endif // USING_INSTRUMENTATION