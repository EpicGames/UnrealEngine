// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/CoreMiscDefines.h"

#if USING_INSTRUMENTATION

#include "CoreTypes.h"

#include "Instrumentation/Types.h"
#include "Instrumentation/Containers.h"
#include "Async/Fundamental/Scheduler.h"
#include "HAL/PlatformStackWalk.h"
#include "Hash/CityHash.h"

#define INSTRUMENTATION_ENUM_CLASS_FLAGS(Enum) \
	INSTRUMENTATION_FUNCTION_ATTRIBUTES FORCEINLINE           Enum& operator|=(Enum& Lhs, Enum Rhs) { return Lhs = (Enum)((__underlying_type(Enum))Lhs | (__underlying_type(Enum))Rhs); } \
	INSTRUMENTATION_FUNCTION_ATTRIBUTES FORCEINLINE           Enum& operator&=(Enum& Lhs, Enum Rhs) { return Lhs = (Enum)((__underlying_type(Enum))Lhs & (__underlying_type(Enum))Rhs); } \
	INSTRUMENTATION_FUNCTION_ATTRIBUTES FORCEINLINE           Enum& operator^=(Enum& Lhs, Enum Rhs) { return Lhs = (Enum)((__underlying_type(Enum))Lhs ^ (__underlying_type(Enum))Rhs); } \
	INSTRUMENTATION_FUNCTION_ATTRIBUTES FORCEINLINE constexpr Enum  operator| (Enum  Lhs, Enum Rhs) { return (Enum)((__underlying_type(Enum))Lhs | (__underlying_type(Enum))Rhs); } \
	INSTRUMENTATION_FUNCTION_ATTRIBUTES FORCEINLINE constexpr Enum  operator& (Enum  Lhs, Enum Rhs) { return (Enum)((__underlying_type(Enum))Lhs & (__underlying_type(Enum))Rhs); } \
	INSTRUMENTATION_FUNCTION_ATTRIBUTES FORCEINLINE constexpr Enum  operator^ (Enum  Lhs, Enum Rhs) { return (Enum)((__underlying_type(Enum))Lhs ^ (__underlying_type(Enum))Rhs); } \
	INSTRUMENTATION_FUNCTION_ATTRIBUTES FORCEINLINE constexpr bool  operator! (Enum  E)             { return !(__underlying_type(Enum))E; } \
	INSTRUMENTATION_FUNCTION_ATTRIBUTES FORCEINLINE constexpr Enum  operator~ (Enum  E)             { return (Enum)~(__underlying_type(Enum))E; }

namespace UE::Sanitizer {

	struct FLocation {
		INSTRUMENTATION_FUNCTION_ATTRIBUTES FLocation(const FString& ModuleName, const FString& FunctionName, const FString& Filename, uint32 Line, UPTRINT ProgramCounter)
			: ModuleName(ModuleName), FunctionName(FunctionName), Filename(Filename), Line(Line), ProgramCounter(ProgramCounter)
		{
		}

		const FString ModuleName;
		const FString FunctionName;
		const FString Filename;
		const uint32  Line;
		const UPTRINT ProgramCounter;

		INSTRUMENTATION_FUNCTION_ATTRIBUTES int32 GetAlignment() const
		{
			if (Filename.Len() == 0)
			{
				return ModuleName.Len() - 3 /* Remove 1 space and () for line number */ ;
			}
			return Filename.Len() + GetDigits(Line);
		}

		INSTRUMENTATION_FUNCTION_ATTRIBUTES FString ToString(int32 Alignment = 0) const
		{
			Alignment = FMath::Max(0, Alignment - GetAlignment());

			if (Filename.Len() == 0)
			{
				// Format in a way that visual studio can understand so we can click the file and goto source.
				return FString::Printf(TEXT("%s:%s %s 0x%p"), *ModuleName, FCString::Spc(Alignment), FunctionName.Len() == 0 ? TEXT("[Unknown Function]") : *FunctionName, (void*)ProgramCounter);
			}

			// Format in a way that visual studio can understand so we can click the file and goto source.
			return FString::Printf(TEXT("%s (%d):%s %s 0x%p"), *Filename, Line, FCString::Spc(Alignment), FunctionName.Len() == 0 ? TEXT("[Unknown Function]") : *FunctionName, (void*)ProgramCounter);
		}

	private:
		INSTRUMENTATION_FUNCTION_ATTRIBUTES static int32 GetDigits(int32 Value)
		{
			int32 Digits = 0;
			do
			{
				Value /= 10;
				++Digits;
			} while (Value != 0);

			return Digits;
		};
	};

	struct FFullLocation
	{
		TArray<FLocation> Locations;

		INSTRUMENTATION_FUNCTION_ATTRIBUTES int32 Num() const
		{ 
			return Locations.Num(); 
		}

		INSTRUMENTATION_FUNCTION_ATTRIBUTES void Reserve(int32 Size) 
		{ 
			Locations.Reserve(Size);
		}

		INSTRUMENTATION_FUNCTION_ATTRIBUTES int32 GetAlignment() const
		{
			int32 MaxAlign = 0;
			for (const FLocation& Location : Locations)
			{
				MaxAlign = FMath::Max(MaxAlign, Location.GetAlignment());
			}

			return MaxAlign;
		}

		INSTRUMENTATION_FUNCTION_ATTRIBUTES FString ToString(int32 Alignment = 0) const
		{
			if (Alignment == 0)
			{
				Alignment = GetAlignment();
			}

			TStringBuilder<4096> Buffer;
			for (int32 Index = 0; Index < Locations.Num();)
			{
				const FLocation& Location = Locations[Index];
				Buffer.Append(Location.ToString(Alignment));
				if (++Index < Locations.Num())
				{
					Buffer.AppendChar(TEXT('\n'));
				}
			}
			return Buffer.ToString();
		}
	};


	// ------------------------------------------------------------------------------
	// Callstack.
	// ------------------------------------------------------------------------------
	class FCallstackLocation 
	{
	public:
		INSTRUMENTATION_FUNCTION_ATTRIBUTES FCallstackLocation()
		{
		}

		INSTRUMENTATION_FUNCTION_ATTRIBUTES FCallstackLocation(void* const* InCallstack, uint32 NumFrames)
		{
			Callstack.Append(InCallstack, NumFrames);
		}

		INSTRUMENTATION_FUNCTION_ATTRIBUTES FFullLocation GetFullLocation() const
		{
			FFullLocation Loc;
			if (Callstack.IsEmpty())
			{
				return Loc;
			}

			// Resolving symbols can cause us to enter a wait that could try to start a new thread
			// and wait for it to be started. We can't allow that since we could deadlock
			// if we're reporting a race while having a lock that the new thread might also need
			// during its initialization. (i.e. Registering new FNames)
			LowLevelTasks::Private::FOversubscriptionAllowedScope AllowOversubscription(false);

			for (int32 Idx = Callstack.Num() - 1; Idx >= 0; --Idx)
			{
				UPTRINT Frame = (UPTRINT)Callstack[Idx];

				const bool bIncludeInlineFrames = true;
				// We always record the return address of our functions using FuncEntry. What we need to properly decode
				// inline frames is something that points on the address of the actual call, not the return address, otherwise
				// we end up decoding inline frames for what comes after we return from the function, which doesn't make sense.
				// By using Frame - 1, we make sure that we actually point inside the chunk of assembly that represents the function call and it is enough
				// to make symbol decode correct since we only need to point inside a valid range of a symbol entry, not directly at the start of it.
				FPlatformStackWalk::EnumerateSymbolInfosForProgramCounter(Frame - 1, bIncludeInlineFrames,
					[&Loc, Frame](FProgramCounterSymbolInfo& SymbolInfo) INSTRUMENTATION_FUNCTION_ATTRIBUTES
					{
						Loc.Locations.Emplace(FString(SymbolInfo.ModuleName), FString(SymbolInfo.FunctionName), FString(SymbolInfo.Filename), SymbolInfo.LineNumber, Frame);
					}
				);
			}

			return Loc;
		}

		INSTRUMENTATION_FUNCTION_ATTRIBUTES uint32 GetHash() const
		{
			if (Callstack.IsEmpty())
			{
				return 0;
			}

			return CityHash32((const char*)Callstack.GetData(), sizeof(void*) * Callstack.Num());
		}

		INSTRUMENTATION_FUNCTION_ATTRIBUTES UPTRINT GetLastFrame() const
		{
			return Callstack.IsEmpty() ? 0 : (UPTRINT)Callstack.Last();
		}

	private:
		// Order is outer frames to inner frames.
		// For example, Main -> Fn1 -> Fn2 -> FnLeaf.
		TArray<void*, TInlineAllocator<1024>> Callstack;
	};

} // namespace UE::Sanitizer

#endif