// Copyright Epic Games, Inc. All Rights Reserved.

#include "AndroidCallstackTrace.h"
#include "ProfilingDebugging/CallstackTrace.h"

#if UE_CALLSTACK_TRACE_ENABLED

#ifndef UE_CALLSTACK_TRACE_ANDROID_USE_STACK_FRAMES_WALKING
	#define UE_CALLSTACK_TRACE_ANDROID_USE_STACK_FRAMES_WALKING 0
#endif

#include "CoreTypes.h"
#include "HAL/Platform.h"
#include "HAL/PlatformStackWalk.h"
#include "ProfilingDebugging/CallstackTracePrivate.h"

////////////////////////////////////////////////////////////////////////////////
class FBacktracer
{
public:
	FBacktracer(FMalloc* InMalloc);
	~FBacktracer();
	static FBacktracer*	Get();
	FORCEINLINE uint32 GetBacktraceId(uint64* StackFrames, uint32 NumStackFrames);
	FORCEINLINE uint32 GetBacktraceId(void* ReturnAddress);

private:
	static FBacktracer* Instance;
	FMalloc* Malloc;
	FCallstackTracer CallstackTracer;
};

////////////////////////////////////////////////////////////////////////////////
FBacktracer* FBacktracer::Instance = nullptr;

////////////////////////////////////////////////////////////////////////////////
FBacktracer::FBacktracer(FMalloc* InMalloc)
	: Malloc(InMalloc)
	, CallstackTracer(InMalloc)
{
	Instance = this;
}

////////////////////////////////////////////////////////////////////////////////
FBacktracer::~FBacktracer()
{
}

////////////////////////////////////////////////////////////////////////////////
FBacktracer* FBacktracer::Get()
{
	return Instance;
}

////////////////////////////////////////////////////////////////////////////////
FORCEINLINE uint32 FBacktracer::GetBacktraceId(uint64* StackFrames, uint32 NumStackFrames)
{
#if !UE_BUILD_SHIPPING
	if (NumStackFrames == 0)
	{
		return 0;
	}

	FCallstackTracer::FBacktraceEntry BacktraceEntry;
	uint64 BacktraceId = 0;
	for (int32 Index = 0; Index < NumStackFrames; Index++)
	{
		// This is a simple order-dependent LCG. Should be sufficient enough
		BacktraceId += StackFrames[Index];
		BacktraceId *= 0x30be8efa499c249dull;
	}

	// Save the collected id
	BacktraceEntry.Hash = BacktraceId;
	BacktraceEntry.FrameCount = NumStackFrames;
	BacktraceEntry.Frames = StackFrames;

	// Add to queue to be processed. This might block until there is room in the
	// queue (i.e. the processing thread has caught up processing).
	return CallstackTracer.AddCallstack(BacktraceEntry);
#else
	return 0;
#endif
}

////////////////////////////////////////////////////////////////////////////////
FORCEINLINE uint32 FBacktracer::GetBacktraceId(void* ReturnAddress)
{
#if !UE_BUILD_SHIPPING
	uint64 StackFrames[UE_CALLSTACK_TRACE_MAX_FRAMES];

#if UE_CALLSTACK_TRACE_ANDROID_USE_STACK_FRAMES_WALKING
	uint32 NumStackFrames = FPlatformStackWalk::CaptureStackBackTraceViaFramePointerWalking(StackFrames, UE_ARRAY_COUNT(StackFrames));
#else
	uint32 NumStackFrames = FPlatformStackWalk::CaptureStackBackTrace(StackFrames, UE_ARRAY_COUNT(StackFrames));
#endif

	if (NumStackFrames > 0)
	{
		// Skip until we find ReturnAddress in the stack, otherwise start from the beginning.
		uint32 StartFromIndex = 0;
		for (int32 Index = 0; Index < NumStackFrames; Index++)
		{
			if (StackFrames[Index] == (uint64)ReturnAddress)
			{
				StartFromIndex = Index;
				break;
			}
		}

		return GetBacktraceId(StackFrames + StartFromIndex, NumStackFrames - StartFromIndex);
	}
#endif

	return 0;
}

////////////////////////////////////////////////////////////////////////////////
void Modules_Initialize();

////////////////////////////////////////////////////////////////////////////////
void CallstackTrace_CreateInternal(FMalloc* Malloc)
{
	if (FBacktracer::Get() != nullptr)
	{
		return;
	}

	// Allocate, construct and intentionally leak backtracer
	void* Alloc = Malloc->Malloc(sizeof(FBacktracer), alignof(FBacktracer));
	new (Alloc) FBacktracer(Malloc);
}

////////////////////////////////////////////////////////////////////////////////
void CallstackTrace_InitializeInternal()
{
	if (FBacktracer* Instance = FBacktracer::Get())
	{
#if !UE_BUILD_SHIPPING
		Modules_Initialize();
#endif
	}
}

////////////////////////////////////////////////////////////////////////////////
uint32 CallstackTrace_GetCurrentId()
{
	if (!UE_TRACE_CHANNELEXPR_IS_ENABLED(CallstackChannel))
	{
		return 0;
	}

	void* ReturnAddress = PLATFORM_RETURN_ADDRESS_FOR_CALLSTACKTRACING();
	if (FBacktracer* Instance = FBacktracer::Get())
	{
		return Instance->GetBacktraceId(ReturnAddress);
	}

	return 0;
}

////////////////////////////////////////////////////////////////////////////////
uint32 CallstackTrace_GetExternalCallstackId(uint64* Frames, uint32 Count)
{
	if (FBacktracer* Instance = FBacktracer::Get())
	{
		return Instance->GetBacktraceId(Frames, Count);
	}

	return 0;
}

#endif