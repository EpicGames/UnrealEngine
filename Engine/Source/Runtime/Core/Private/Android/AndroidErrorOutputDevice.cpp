// Copyright Epic Games, Inc. All Rights Reserved.

#include "Android/AndroidErrorOutputDevice.h"

#include "CoreGlobals.h"
#include "Misc/OutputDeviceHelper.h"
#include "HAL/PlatformMisc.h"
#include "Misc/OutputDeviceRedirector.h"
#include "CoreGlobals.h"
#include "Misc/CoreDelegates.h"

FAndroidErrorOutputDevice::FAndroidErrorOutputDevice()
{
}

void FAndroidErrorOutputDevice::Serialize( const TCHAR* Msg, ELogVerbosity::Type Verbosity, const class FName& Category )
{
	FPlatformMisc::LowLevelOutputDebugString(*FOutputDeviceHelper::FormatLogLine(Verbosity, Category, Msg, GPrintLogTimes));

	static int32 CallCount = 0;
	int32 NewCallCount = FPlatformAtomics::InterlockedIncrement(&CallCount);
	if(GIsCriticalError == 0 && NewCallCount == 1)
	{
		// First appError.
		GIsCriticalError = 1;

		FCString::Strncpy(GErrorExceptionDescription, Msg, UE_ARRAY_COUNT(GErrorExceptionDescription));
	}
	else
	{
		UE_LOGF(LogAndroid, Error, "Error reentered: %ls", Msg);
	}

	if (GIsGuarded)
	{
		UE_DEBUG_BREAK();
	}
	else
	{
		HandleError();
		FPlatformMisc::RequestExit(true, TEXT("FAndroidErrorOutputDevice::Serialize.!GIsGuarded"));
	}
}

void FAndroidErrorOutputDevice::HandleError()
{
	static int32 CallCount = 0;
	int32 NewCallCount = FPlatformAtomics::InterlockedIncrement(&CallCount);

	if (NewCallCount != 1)
	{
		UE_LOGF(LogAndroid, Error, "HandleError re-entered.");
		return;
	}
	
	GIsGuarded = 0;
	GIsRunning = 0;
	GIsCriticalError = 1;
	GLogConsole = NULL;
	GErrorHist[UE_ARRAY_COUNT(GErrorHist) - 1] = 0;

	// Dump the error and flush the log.
#if !NO_LOGGING
	FDebug::LogFormattedMessageWithCallstack(LogAndroid.GetCategoryName(), __FILE__, __LINE__, TEXT("=== Critical error: ==="), GErrorHist, ELogVerbosity::Error);
#endif
	
	GLog->Panic();

	FCoreDelegates::OnHandleSystemError.Broadcast();
	FCoreDelegates::OnShutdownAfterError.Broadcast();
}
