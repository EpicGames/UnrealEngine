// Copyright Epic Games, Inc. All Rights Reserved.

#include "Android/AndroidHeapProfiling.h"


#if ANDROID_HEAP_PROFILING_SUPPORTED
	#include <dlfcn.h>
	#include "Misc/CString.h"
	#include "Containers/StringConv.h"

	struct AHeapInfo;
	AHeapInfo* (*AHeapInfoCreate)(const char* heap_name) = nullptr;
	uint32_t(*AHeapProfileRegisterHeap)(AHeapInfo* info) = nullptr;
	bool (*AHeapProfileReportAllocation)(uint32_t heap_id, uint64_t alloc_id, uint64_t size) = nullptr;
	void (*AHeapProfileReportFree)(uint32_t heap_id, uint64_t alloc_id) = nullptr;

	static bool LoadSymbol(void* Module, void** FuncPtr, const char* SymbolName)
	{
		*FuncPtr = dlsym(Module, SymbolName);
		if (!*FuncPtr)
		{
			FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Cannot locate symbol `%s` in heapprofd_standalone_client.so"), ANSI_TO_TCHAR(SymbolName));
			return false;
		}

		return true;
	}

	uint32_t CreateHeap(const TCHAR* AllocatorName)
	{
		const int AllocatorNameBufferSize = 256;
		char AllocatorNameBuffer[AllocatorNameBufferSize] = "epicgames.";
		const int Len = FCStringAnsi::Strlen(AllocatorNameBuffer);
		FCStringAnsi::Strncpy(AllocatorNameBuffer + Len, TCHAR_TO_ANSI(AllocatorName), AllocatorNameBufferSize - Len);
		return AHeapProfileRegisterHeap(AHeapInfoCreate(AllocatorNameBuffer));
	}
#endif


bool AndroidHeapProfiling::Init()
{
#if ANDROID_HEAP_PROFILING_SUPPORTED
	const int32 OSVersion = android_get_device_api_level();
	const int32 Android10Level = __ANDROID_API_Q__;
	if (OSVersion >= Android10Level)
	{
		void* HeapprofdClient = dlopen("libheapprofd_standalone_client.so", 0);
		if (HeapprofdClient)
		{
			bool InitSuccessful = LoadSymbol(HeapprofdClient, (void**)(&AHeapInfoCreate), "AHeapInfo_create");
			InitSuccessful &= LoadSymbol(HeapprofdClient, (void**)(&AHeapProfileRegisterHeap), "AHeapProfile_registerHeap");
			InitSuccessful &= LoadSymbol(HeapprofdClient, (void**)(&AHeapProfileReportAllocation), "AHeapProfile_reportAllocation");
			InitSuccessful &= LoadSymbol(HeapprofdClient, (void**)(&AHeapProfileReportFree), "AHeapProfile_reportFree");

			if (!InitSuccessful)
			{
				dlclose(HeapprofdClient);
				AHeapInfoCreate = nullptr;
				AHeapProfileRegisterHeap = nullptr;
				AHeapProfileReportAllocation = nullptr;
				AHeapProfileReportFree = nullptr;
			}

			return InitSuccessful;
		}
		else
		{
			const char* Error = dlerror();
			char ErrorBuf[DEFAULT_STRING_CONVERSION_SIZE];
			FCStringAnsi::Strncpy(ErrorBuf, Error, DEFAULT_STRING_CONVERSION_SIZE);
			FPlatformMisc::LocalPrint(ANSI_TO_TCHAR(ErrorBuf));
		}
	}
#endif
	return false;
}