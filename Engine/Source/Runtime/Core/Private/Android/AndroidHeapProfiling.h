// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifndef ANDROID_HEAP_PROFILING_SUPPORTED
#	define ANDROID_HEAP_PROFILING_SUPPORTED 0
#endif

#if ANDROID_HEAP_PROFILING_SUPPORTED
	#include <type_traits>
	#include "HAL/MallocAnsi.h"

	uint32_t CreateHeap(const TCHAR* AllocatorName);
	extern bool (*AHeapProfileReportAllocation)(uint32_t heap_id, uint64_t alloc_id, uint64_t size);
	extern void (*AHeapProfileReportFree)(uint32_t heap_id, uint64_t alloc_id);
#endif

template <class T>
struct FMallocProfilingProxy : public T
{
#if ANDROID_HEAP_PROFILING_SUPPORTED
	FMallocProfilingProxy()
	{
		static_assert(!std::is_same<T, FMallocAnsi>::value, "FMallocProfilingProxy should never be parametrized with FMallocAnsi since FMallocAnsi will be hooked by heapprofd internally");
		HeapId = CreateHeap(T::GetDescriptiveName());
	}

	virtual void* Malloc(SIZE_T Size, uint32 Alignment) final
	{
		void* Ptr = T::Malloc(Size, Alignment);
		AHeapProfileReportAllocation(HeapId, (uint64_t)Ptr, Size);
		return Ptr;
	}

	virtual void* TryMalloc(SIZE_T Size, uint32 Alignment) final
	{
		void* Ptr = T::Malloc(Size, Alignment);
		if (Ptr)
		{
			AHeapProfileReportAllocation(HeapId, (uint64_t)Ptr, Size);
		}

		return Ptr;
	}

	virtual void* Realloc(void* Ptr, SIZE_T NewSize, uint32 Alignment) final
	{
		if (Ptr != nullptr)
		{
			AHeapProfileReportFree(HeapId, (uint64_t)Ptr);
		}

		Ptr = T::Realloc(Ptr, NewSize, Alignment);

		if (Ptr != nullptr)
		{
			AHeapProfileReportAllocation(HeapId, (uint64_t)Ptr, NewSize);
		}

		return Ptr;
	}

	virtual void* TryRealloc(void* Ptr, SIZE_T NewSize, uint32 Alignment) final
	{
		return Realloc(Ptr, NewSize, Alignment);
	}

	virtual void Free(void* Ptr) final
	{
		AHeapProfileReportFree(HeapId, (uint64_t)Ptr);
		T::Free(Ptr);
	}

private:
	uint32_t HeapId;
#endif
};


struct AndroidHeapProfiling
{
	static bool Init();
};