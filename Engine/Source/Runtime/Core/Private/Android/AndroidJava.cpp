// Copyright Epic Games, Inc. All Rights Reserved.

#include "Android/AndroidJava.h"
#include "Android/AndroidJniGameActivity.h"

#if USE_ANDROID_JNI

FJavaClassObject FJavaClassObject::GetGameActivity()
{
	return {UE::Jni::Class<UE::Jni::FGameActivity>, AndroidJavaEnv::GetGameActivityThis()};
}

FJavaClassObject::FJavaClassObject(FName ClassName, const char* CtorSig, ...)
{
	JNIEnv*	JEnv = AndroidJavaEnv::GetJavaEnv();

	ANSICHAR AnsiClassName[NAME_SIZE];
	ClassName.GetPlainANSIString(AnsiClassName);

	Class = AndroidJavaEnv::FindJavaClassGlobalRef(AnsiClassName);
	check(Class);
	jmethodID Constructor = JEnv->GetMethodID(Class, "<init>", CtorSig);
	check(Constructor);
	va_list Params;
	va_start(Params, CtorSig);
	auto LocalObject = NewScopedJavaObject(JEnv, JEnv->NewObjectV(Class, Constructor, Params));
	va_end(Params);
	VerifyException();
	check(LocalObject);

	// Promote local references to global
	Object = JEnv->NewGlobalRef(*LocalObject);
}

FJavaClassObject::FJavaClassObject(jclass LocalClass, jobject LocalObject)
{
	check(LocalClass);
	check(LocalObject);

	Class = UE::Jni::Env.NewGlobalRef(LocalClass);
	Object = UE::Jni::Env.NewGlobalRef(LocalObject);
}

FJavaClassObject::~FJavaClassObject()
{
	JNIEnv*	JEnv = AndroidJavaEnv::GetJavaEnv();
	JEnv->DeleteGlobalRef(Object);
	JEnv->DeleteGlobalRef(Class);
}

FJavaClassMethod FJavaClassObject::GetClassMethod(const char* MethodName, const char* FuncSig)
{
	JNIEnv*	JEnv = AndroidJavaEnv::GetJavaEnv();
	FJavaClassMethod Method;
	Method.Method = JEnv->GetMethodID(Class, MethodName, FuncSig);
	Method.Name = MethodName;
	Method.Signature = FuncSig;
	// Is method valid?
	checkf(Method.Method, TEXT("Unable to find Java Method %s with Signature %s"), UTF8_TO_TCHAR(MethodName), UTF8_TO_TCHAR(FuncSig));
	return Method;
}

template<>
void FJavaClassObject::CallMethod<void>(FJavaClassMethod Method, ...)
{
	JNIEnv*	JEnv = AndroidJavaEnv::GetJavaEnv();
	va_list Params;
	va_start(Params, Method);
	JEnv->CallVoidMethodV(Object, Method.Method, Params);
	va_end(Params);
	VerifyException();
}

template<>
bool FJavaClassObject::CallMethod<bool>(FJavaClassMethod Method, ...)
{
	JNIEnv*	JEnv = AndroidJavaEnv::GetJavaEnv();
	va_list Params;
	va_start(Params, Method);
	bool RetVal = JEnv->CallBooleanMethodV(Object, Method.Method, Params);
	va_end(Params);
	VerifyException();
	return RetVal;
}

template<>
int FJavaClassObject::CallMethod<int>(FJavaClassMethod Method, ...)
{
	JNIEnv*	JEnv = AndroidJavaEnv::GetJavaEnv();
	va_list Params;
	va_start(Params, Method);
	int RetVal = JEnv->CallIntMethodV(Object, Method.Method, Params);
	va_end(Params);
	VerifyException();
	return RetVal;
}

template<>
jobject FJavaClassObject::CallMethod<jobject>(FJavaClassMethod Method, ...)
{
	JNIEnv*	JEnv = AndroidJavaEnv::GetJavaEnv();
	va_list Params;
	va_start(Params, Method);
	jobject val = JEnv->CallObjectMethodV(Object, Method.Method, Params);
	va_end(Params);
	VerifyException();
	jobject RetVal = JEnv->NewGlobalRef(val);
	JEnv->DeleteLocalRef(val);
	return RetVal;
}

template<>
jobjectArray FJavaClassObject::CallMethod<jobjectArray>(FJavaClassMethod Method, ...)
{
	JNIEnv*	JEnv = AndroidJavaEnv::GetJavaEnv();
	va_list Params;
	va_start(Params, Method);
	jobject val = JEnv->CallObjectMethodV(Object, Method.Method, Params);
	va_end(Params);
	VerifyException();
	jobjectArray RetVal = (jobjectArray)JEnv->NewGlobalRef(val);
	JEnv->DeleteLocalRef(val);
	return RetVal;
}

template<>
int64 FJavaClassObject::CallMethod<int64>(FJavaClassMethod Method, ...)
{
	JNIEnv*	JEnv = AndroidJavaEnv::GetJavaEnv();
	va_list Params;
	va_start(Params, Method);
	int64 RetVal = JEnv->CallLongMethodV(Object, Method.Method, Params);
	va_end(Params);
	VerifyException();
	return RetVal;
}

template<>
FString FJavaClassObject::CallMethod<FString>(FJavaClassMethod Method, ...)
{
	JNIEnv*	JEnv = AndroidJavaEnv::GetJavaEnv();
	va_list Params;
	va_start(Params, Method);
	jstring RetVal = static_cast<jstring>(
		JEnv->CallObjectMethodV(Object, Method.Method, Params));
	va_end(Params);
	VerifyException();
	auto Result = FJavaHelper::FStringFromLocalRef(JEnv, RetVal);
	return Result;
}

template<>
float FJavaClassObject::CallMethod<float>(FJavaClassMethod Method, ...)
{
	JNIEnv* JEnv = AndroidJavaEnv::GetJavaEnv();
	va_list Params;
	va_start(Params, Method);
	float RetVal = JEnv->CallFloatMethodV(Object, Method.Method, Params);
	va_end(Params);
	VerifyException();
	return RetVal;
}

FScopedJavaObject<jstring> FJavaClassObject::GetJString(const FString& String)
{
	JNIEnv*	JEnv = AndroidJavaEnv::GetJavaEnv();
	return FJavaHelper::ToJavaString(JEnv, String);
}

void FJavaClassObject::VerifyException()
{
	JNIEnv*	JEnv = AndroidJavaEnv::GetJavaEnv();
	if (JEnv->ExceptionCheck())
	{
		JEnv->ExceptionDescribe();
		JEnv->ExceptionClear();
		verify(false && "Java JNI call failed with an exception.");
	}
}

#endif
