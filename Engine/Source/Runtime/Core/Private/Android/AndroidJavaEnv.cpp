// Copyright Epic Games, Inc. All Rights Reserved.

#if USE_ANDROID_JNI
#include "Android/AndroidJavaEnv.h"
#include "Android/AndroidJniGameActivity.h"
#include "CoreMinimal.h"
#include "HAL/ThreadSingleton.h"
#include "HAL/PlatformMisc.h"
#include "HAL/ThreadManager.h"
#include "Containers/UnrealString.h"

//////////////////////////////////////////////////////////////////////////
JavaVM* GJavaVM;
static UE::Jni::Java::Lang::FClassLoader* ClassLoader;

void AndroidJavaEnv::InitializeJavaEnv(JavaVM* VM, jint Version, jobject GlobalThis)
{
	InitializeJavaEnv();
}

void AndroidJavaEnv::InitializeJavaEnv()
{
	UE::Jni::Env.Initialize();
}

jobject AndroidJavaEnv::GetClassLoader()
{
	return ClassLoader;
}

JNIEnv* AndroidJavaEnv::GetJavaEnv( bool bRequireGlobalThis /*= true*/ )
{
	return UE::Jni::Env;
}

jclass AndroidJavaEnv::FindJavaClass(const char* name)
{
	JNIEnv* Env = GetJavaEnv();
	if (!Env)
	{
		return nullptr;
	}

	jstring ClassNameObj = Env->NewStringUTF(name);
	jclass FoundClass = UE::Jni::Java::Lang::FClassLoader::loadClass(ClassLoader, ClassNameObj).Leak();

	if (Env->ExceptionCheck())
	{
		// Clear exception because we failed to find the class and try again using JNIEnv
		Env->ExceptionClear();

		FoundClass = static_cast<jclass>(Env->FindClass(name));
		CheckJavaException();
	}
	Env->DeleteLocalRef(ClassNameObj);
	
	return FoundClass;
}

jclass AndroidJavaEnv::FindJavaClassGlobalRef(const char* name)
{
	JNIEnv* Env = GetJavaEnv();
	if (!Env)
	{
		return nullptr;
	}

	jstring ClassNameObj = Env->NewStringUTF(name);
	auto FoundClass = UE::Jni::Java::Lang::FClassLoader::loadClass(ClassLoader, ClassNameObj);
	
	if (Env->ExceptionCheck())
	{
		// Clear exception because we failed to find the class and try again using JNIEnv
		Env->ExceptionClear();

		FoundClass = NewScopedJavaObject(Env, static_cast<jclass>(Env->FindClass(name)));
		CheckJavaException();
	}
	Env->DeleteLocalRef(ClassNameObj);

	auto GlobalClass = (jclass)Env->NewGlobalRef(*FoundClass);
	return GlobalClass;
}

void AndroidJavaEnv::DetachJavaEnv()
{
	GJavaVM->DetachCurrentThread();
}

bool AndroidJavaEnv::CheckJavaException()
{
	JNIEnv* Env = GetJavaEnv();
	if (!Env)
	{
		return true;
	}
	if (Env->ExceptionCheck())
	{
		Env->ExceptionDescribe();
		Env->ExceptionClear();
		verify(false && "Java JNI call failed with an exception.");
		return true;
	}
	return false;
}

FString FJavaHelper::FStringFromLocalRef(JNIEnv* Env, jstring JavaString)
{
	FString ReturnString = FStringFromParam(Env, JavaString);
	
	if (Env && JavaString)
	{
		Env->DeleteLocalRef(JavaString);
	}
	
	return ReturnString;
}

FString FJavaHelper::FStringFromGlobalRef(JNIEnv* Env, jstring JavaString)
{
	FString ReturnString = FStringFromParam(Env, JavaString);
	
	if (Env && JavaString)
	{
		Env->DeleteGlobalRef(JavaString);
	}
	
	return ReturnString;
}

FString FJavaHelper::FStringFromParam(JNIEnv* Env, jstring JavaString)
{
	if (!Env || !JavaString || Env->IsSameObject(JavaString, NULL))
	{
		return {};
	}
	
	const auto chars = Env->GetStringUTFChars(JavaString, 0);
	FString ReturnString(UTF8_TO_TCHAR(chars));
	Env->ReleaseStringUTFChars(JavaString, chars);
	return ReturnString;
}

FScopedJavaObject<jstring> FJavaHelper::ToJavaString(JNIEnv* Env, const FString& UnrealString)
{
	check(Env);
	return NewScopedJavaObject(Env, Env->NewStringUTF(TCHAR_TO_UTF8(*UnrealString)));
}

FScopedJavaObject<jobjectArray> FJavaHelper::ToJavaStringArray(JNIEnv* Env, const TArray<FStringView>& UnrealStrings)
{
	jobjectArray ObjectArray = Env->NewObjectArray((jsize)UnrealStrings.Num(), UE::Jni::Class<UE::Jni::Java::Lang::FString>, NULL);
	for (int32 Idx = 0; Idx < UnrealStrings.Num(); ++Idx)
	{
		// FStringView of an empty FString contains a null pointer as data
		if (UnrealStrings[Idx].GetData())
		{
			Env->SetObjectArrayElement(ObjectArray, Idx, *FScopedJavaObject{Env->NewStringUTF(TCHAR_TO_UTF8(UnrealStrings[Idx].GetData()))});
		}
		else
		{		
			Env->SetObjectArrayElement(ObjectArray, Idx, *FScopedJavaObject{Env->NewStringUTF("")});
		}
	}
	return NewScopedJavaObject(Env, ObjectArray);
}

TArray<FString> FJavaHelper::ObjectArrayToFStringTArray(JNIEnv* Env, jobjectArray ObjectArray)
{
	TArray<FString> ArrayOfStrings;
	if (Env && ObjectArray && !Env->IsSameObject(ObjectArray, NULL))
	{
		jsize Size = Env->GetArrayLength(ObjectArray);

		ArrayOfStrings.Reserve(Size);

		for (jsize Idx = 0; Idx < Size; ++Idx)
		{
			FString Entry = FStringFromLocalRef(Env, (jstring)Env->GetObjectArrayElement(ObjectArray, Idx));
			ArrayOfStrings.Add(MoveTemp(Entry));
		}
	}
	return ArrayOfStrings;
}

namespace UE::Jni
{
	FEnv::FEnv()
	{
		check(GJavaVM);
        
		if (jint Result = GJavaVM->GetEnv((void**)&Value, JNI_VERSION_1_6); Result == JNI_EDETACHED)
		{
			bAttached = true;
            
			uint32 ThreadId = FPlatformTLS::GetCurrentThreadId();
			auto ThreadName = StringCast<char>(*FThreadManager::GetThreadName(ThreadId));
			JavaVMAttachArgs Args{JNI_VERSION_1_6, ThreadName.Length() ? ThreadName.Get() : FAndroidMisc::GetThreadName(ThreadId)};
    
			if (Result = GJavaVM->AttachCurrentThread(&Value, &Args); Result)
			{
				FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FEnv failed to attach thread! Result = %d"), Result);
				check(false);
			}
		}
		else
		{
			bAttached = false;
            
			if (Result)
			{
				FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FEnv failed to get JNI environment! Result = %d"), Result);
				check(false);
			}
		}
	}
	
	FEnv::~FEnv()
	{
		if (bAttached)
		{
			check(GJavaVM);
            
			if (jint Result = GJavaVM->DetachCurrentThread())
			{
				FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FEnv failed to detach thread! Result = %d"), Result);
				check(false);
			}
		}    
	}

	FScopedJavaObject<jclass> FEnv::FindClass(const char* ClassName) const
	{
		FScopedJavaObject<jclass> Class = Java::Lang::FClassLoader::loadClass(ClassLoader, *FScopedJavaObject{Value->NewStringUTF(ClassName)});

		if (!Class)
		{
			Value->ExceptionClear();
		}
	
		return Class;
	}
	
	bool FEnv::Register(jclass Class, TArrayView<const FMember> Members) const
	{
		for (const FMember& Member : Members)
		{
			switch (Member.Type)
			{
			case FMember::StaticField:
				{
					checkf(!*Member.FieldId, TEXT("FEnv::Register called more than once"));
				
					jfieldID FieldId = Value->GetStaticFieldID(Class, Member.Name, Member.Signature);
				
					if (!FieldId)
					{
						Value->ExceptionClear();
						return false;
					}

					*Member.FieldId = FieldId;
				}
				break;
			case FMember::Field:
				{
					checkf(!*Member.FieldId, TEXT("FEnv::Register called more than once"));
				
					jfieldID FieldId = Value->GetFieldID(Class, Member.Name, Member.Signature);
				
					if (!FieldId)
					{
						Value->ExceptionClear();
						return false;
					}

					*Member.FieldId = FieldId;
				}
				break;
			case FMember::StaticMethod:
				{
					checkf(!*Member.MethodId, TEXT("FEnv::Register called more than once"));
				
					jmethodID MethodId = Value->GetStaticMethodID(Class, Member.Name, Member.Signature);
				
					if (!MethodId)
					{
						Value->ExceptionClear();
						return false;
					}

					*Member.MethodId = MethodId;
				}
				break;
			case FMember::Method:
				{
					checkf(!*Member.MethodId, TEXT("FEnv::Register called more than once"));
				
					jmethodID MethodId = Value->GetMethodID(Class, Member.Name, Member.Signature);
				
					if (!MethodId)
					{
						Value->ExceptionClear();
						return false;
					}

					*Member.MethodId = MethodId;
				}
				break;
			}
		}

		return true;
	}
	
	bool FEnv::Register(jclass Class, TArrayView<const FNativeMethod> NativeMethods) const
	{
		if (Value->RegisterNatives(Class, NativeMethods.GetData(), jint(NativeMethods.Num())) < 0)
		{
			Value->ExceptionClear();
			return false;
		}

		return true;
	}

	FScopedJavaObject<Java::Lang::FString*> FEnv::Marshal(FStringView Param) const
	{
		return NewScopedJavaObject(Value, Value->NewString(reinterpret_cast<const jchar*>(Param.GetData()), Param.Len()));
	}
	
	FString FEnv::Marshal(Java::Lang::FString* Param) const
	{
		FString String;

		if (Param)
		{
			if (jsize Len = Value->GetStringLength(Param))
			{
				String.GetCharArray().AddUninitialized(Len + 1);
				TCHAR* Buffer = String.GetCharArray().GetData();
				Value->GetStringRegion(Param, 0, Len, reinterpret_cast<jchar*>(Buffer));
				Buffer[Len] = '\0';
			}
		}

		return String;
	}

	extern "C" void (FEnv::*const __start_UE_Jni_Initialize [[gnu::weak]])() const;
    extern "C" void (FEnv::*const __stop_UE_Jni_Initialize [[gnu::weak]])() const;
	void FEnv::Initialize() const
	{
		using namespace Java::Lang;

		Class<FString> = NewGlobalRef(*FScopedJavaObject<TClass<FString>*>{Value->FindClass(ClassName<FString>.GetData())});
	
		Initialize<FClassLoader, false>(*FScopedJavaObject<TClass<FClassLoader>*>{Value->FindClass(ClassName<FClassLoader>.GetData())});
	
		Class<FGameActivity> = NewGlobalRef(*FScopedJavaObject<TClass<FGameActivity>*>{Value->FindClass(ClassName<FGameActivity>.GetData())});
		Initialize<FGameActivity, false>(Class<FGameActivity>);

		ClassLoader = NewGlobalRef(*Get<FClassLoader* ()>(*FScopedJavaObject<TClass<FClass>*>{Value->FindClass(ClassName<FClass>.GetData())}, "getClassLoader")(Class<FGameActivity>));
		
		Initialize<FInteger>();
		Initialize<Java::Util::FHashMap>();

		for (auto* Initialize = &__start_UE_Jni_Initialize; Initialize < &__stop_UE_Jni_Initialize; ++Initialize)
		{
			(this->**Initialize)();
		}
	}
}
#endif