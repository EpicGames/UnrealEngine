// Copyright Epic Games, Inc. All Rights Reserved.

#include "Android/AndroidJavaMessageBox.h"

#if USE_ANDROID_JNI

FJavaAndroidMessageBox::FJavaAndroidMessageBox()
	: FJavaClassObject(GetClassName(), "()V")
	, SetCaptionMethod(GetClassMethod("setCaption", "(Ljava/lang/String;)V"))
	, SetTextMethod(GetClassMethod("setText", "(Ljava/lang/String;)V"))
	, AddButtonMethod(GetClassMethod("addButton", "(Ljava/lang/String;)V"))
	, ClearMethod(GetClassMethod("clear", "()V"))
	, ShowMethod(GetClassMethod("show", "()I"))
{
}

void FJavaAndroidMessageBox::SetCaption(const FString & Text)
{
	CallMethod<void>(SetCaptionMethod, *GetJString(Text));
}

void FJavaAndroidMessageBox::SetText(const FString & Text)
{
	CallMethod<void>(SetTextMethod, *GetJString(Text));
}

void FJavaAndroidMessageBox::AddButton(const FString & Text)
{
	CallMethod<void>(AddButtonMethod, *GetJString(Text));
}

void FJavaAndroidMessageBox::Clear()
{
	CallMethod<void>(ClearMethod);
}

int32 FJavaAndroidMessageBox::Show()
{
	return CallMethod<int32>(ShowMethod);
}

FName FJavaAndroidMessageBox::GetClassName()
{
	if (FAndroidMisc::GetAndroidBuildVersion() >= 1)
	{
		return FName("com/epicgames/unreal/MessageBox01");
	}
	else
	{
		return FName();
	}
}

#endif
