// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#if USE_ANDROID_JNI
#include "Android/AndroidJavaEnv.h"

namespace UE::Jni::AndroidProfiling
{
	struct FProfilerAccessor: Java::Lang::FObject
	{
		static constexpr bool bIsOptional = true;
		
		static constexpr FAnsiStringView ClassName = "com/epicgames/unreal/androidprofiling/ProfilerAccessor";

		static void JNICALL nativeOnProfileFinish(JNIEnv* env, jclass clazz, jstring ProfileNameJava, jstring ProfileErrorJava, jstring ProfileFilepathJava);

		static constexpr FNativeMethod NativeMethods[]
		{
			UE_JNI_NATIVE_METHOD(nativeOnProfileFinish)
		};
	};
}

namespace UE::Jni
{
	template struct TInitialize<AndroidProfiling::FProfilerAccessor>;
}
#endif
