// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProfilingDebugging/MemoryTrace.h"

#if UE_MEMORY_TRACE_ENABLED

#include <android/log.h>
#include <string.h>

////////////////////////////////////////////////////////////////////////////////
FMalloc* MemoryTrace_CreateInternal(FMalloc*, EMemoryTraceInit);

////////////////////////////////////////////////////////////////////////////////
FMalloc* MemoryTrace_Create(FMalloc* InMalloc)
{
	const char* UEEnableMemoryTracing = getenv("UEEnableMemoryTracing");
	__android_log_print(ANDROID_LOG_DEBUG, "UE", "getenv(\"UEEnableMemoryTracing\") == \"%s\"", UEEnableMemoryTracing ? UEEnableMemoryTracing : "nullptr");

	// If environment variable UEEnableMemoryTracing is present memory tracing will be fully active
	// unless equal to "light" in which case the light memory preset is used.
	EMemoryTraceInit Mode = EMemoryTraceInit::Disabled;
	if (UEEnableMemoryTracing != nullptr && !strncmp(UEEnableMemoryTracing, "light", 5))
	{
		Mode = EMemoryTraceInit::Light;
	}
	else if (UEEnableMemoryTracing != nullptr)
	{
		Mode = EMemoryTraceInit::Full;
	}
	return MemoryTrace_CreateInternal(InMalloc, Mode);
}

#endif // UE_MEMORY_TRACE_ENABLED
