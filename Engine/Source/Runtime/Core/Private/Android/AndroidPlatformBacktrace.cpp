// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AndroidPlatformStackWalk.cpp: Android implementations of stack walk functions
=============================================================================*/

#include "CoreMinimal.h"
#include "HAL/UnrealMemory.h"
#include <signal.h>
#include <cxxabi.h>
#include <stdio.h>

#if PLATFORM_ANDROID_ARM

#include <unwind.h>

// missing in r9c llvm-3.3
typedef uintptr_t _Unwind_Ptr;

#ifndef _SYS_UCONTEXT_H_
// missing r9c
typedef struct ucontext {
    uint32_t uc_flags;
    struct ucontext* uc_link;
    stack_t uc_stack;
    struct sigcontext uc_mcontext;
    uint32_t uc_sigmask;
} ucontext_t;
#endif // _SYS_UCONTEXT_H_


/* Unwind state. */
typedef struct {
    uint32_t gregs[16];
} unwind_state_t;

static const int R_SP = 13;
static const int R_LR = 14;
static const int R_PC = 15;

/* Special EXIDX value that indicates that a frame cannot be unwound. */
static const uint32_t EXIDX_CANTUNWIND = 1;
/* Get the EXIDX section start and size for the module that contains a
 * given program counter address.
 *
 * When the executable is statically linked, the EXIDX section can be
 * accessed by querying the values of the __exidx_start and __exidx_end
 * symbols.
 *
 * When the executable is dynamically linked, the linker exports a function
 * called dl_unwind_find_exidx that obtains the EXIDX section for a given
 * absolute program counter address.
 *
 * Bionic exports a helpful function called __gnu_Unwind_Find_exidx that
 * handles both cases, so we use that here.
 */
//extern _Unwind_Ptr __gnu_Unwind_Find_exidx(_Unwind_Ptr pc, int *pcount);
// have to define this since cannot include link.h
__BEGIN_DECLS
_Unwind_Ptr dl_unwind_find_exidx(_Unwind_Ptr pc, int* pcount);
__END_DECLS

static uintptr_t find_exidx(uintptr_t pc, size_t* out_exidx_size) {
	int count;
	uintptr_t start = (uintptr_t)dl_unwind_find_exidx((_Unwind_Ptr)pc, &count);
    *out_exidx_size = count;
    return start;
}

/* Transforms a 31-bit place-relative offset to an absolute address.
 * We assume the most significant bit is clear. */
static uintptr_t prel_to_absolute(uintptr_t place, uint32_t prel_offset) {
    return place + (((int32_t)(prel_offset << 1)) >> 1);
}

static bool try_get_word(uintptr_t ptr, uint32_t* out_value) {
    if (ptr & 3) {
		*out_value = 0xffffffffL;
		return false;
    }
	*out_value = *(uint32_t*)ptr;
	return true;
}

static bool try_get_half_word(uint32_t pc, uint16_t* out_value) {
    uint32_t word;
    if (try_get_word(pc & ~2, &word)) {
        *out_value = pc & 2 ? word >> 16 : word & 0xffff;
        return true;
    }
    return false;
}

static uintptr_t get_exception_handler(uintptr_t pc) {
    if (!pc) {
        return 0;
    }
    
	size_t exidx_size;
    uintptr_t exidx_start = find_exidx(pc, &exidx_size);

    uintptr_t handler = 0;
    int32_t handler_index = -1;
    if (exidx_start) {
        uint32_t low = 0;
        uint32_t high = exidx_size;
        while (low < high) {
            uint32_t index = (low + high) / 2;
            uintptr_t entry = exidx_start + index * 8;
            uint32_t entry_prel_pc;
            if (!try_get_word(entry, &entry_prel_pc)) {
                break;
            }
            uintptr_t entry_pc = prel_to_absolute(entry, entry_prel_pc);
            if (pc < entry_pc) {
                high = index;
                continue;
            }
            if (index + 1 < exidx_size) {
                uintptr_t next_entry = entry + 8;
                uint32_t next_entry_prel_pc;
                if (!try_get_word(next_entry, &next_entry_prel_pc)) {
                    break;
                }
                uintptr_t next_entry_pc = prel_to_absolute(next_entry, next_entry_prel_pc);
                if (pc >= next_entry_pc) {
                    low = index + 1;
                    continue;
                }
            }
            uintptr_t entry_handler_ptr = entry + 4;
            uint32_t entry_handler;
            if (!try_get_word(entry_handler_ptr, &entry_handler)) {
                break;
            }
            if (entry_handler & (1L << 31)) {
                handler = entry_handler_ptr; // in-place handler data
            } else if (entry_handler != EXIDX_CANTUNWIND) {
                handler = prel_to_absolute(entry_handler_ptr, entry_handler);
            }
            handler_index = index;
            break;
        }
    }
 
    return handler;
}
/*
 * Describes a single frame of a backtrace.
 */
typedef struct {
    uintptr_t absolute_pc;     /* absolute PC offset */
    uintptr_t stack_top;       /* top of stack for this frame */
    size_t stack_size;         /* size of this stack frame */
} backtrace_frame_t;

typedef struct {
    uintptr_t ptr;
    uint32_t word;
} byte_stream_t;

static bool try_next_byte(byte_stream_t* stream, uint8_t* out_value) {
    uint8_t result;
    switch (stream->ptr & 3) {
    case 0:
        if (!try_get_word(stream->ptr, &stream->word)) {
            *out_value = 0;
            return false;
        }
        *out_value = stream->word >> 24;
        break;
    case 1:
        *out_value = stream->word >> 16;
        break;
    case 2:
        *out_value = stream->word >> 8;
        break;
    default:
        *out_value = stream->word;
        break;
    }
    stream->ptr += 1;
    return true;
}

static void set_reg(unwind_state_t* state, uint32_t reg, uint32_t value) {
    state->gregs[reg] = value;
}

static bool try_pop_registers(unwind_state_t* state, uint32_t mask) {
    uint32_t sp = state->gregs[R_SP];
    bool sp_updated = false;
    for (int i = 0; i < 16; i++) {
        if (mask & (1 << i)) {
            uint32_t value;
            if (!try_get_word(sp, &value)) {
                return false;
            }
            if (i == R_SP) {
                sp_updated = true;
            }
            set_reg(state, i, value);
            sp += 4;
        }
    }
    
	if (!sp_updated) {
        set_reg(state, R_SP, sp);
    }
    return true;
}

/* Executes a built-in personality routine as defined in the EHABI.
 * Returns true if unwinding should continue.
 *
 * The data for the built-in personality routines consists of a sequence
 * of unwinding instructions, followed by a sequence of scope descriptors,
 * each of which has a length and offset encoded using 16-bit or 32-bit
 * values.
 *
 * We only care about the unwinding instructions.  They specify the
 * operations of an abstract machine whose purpose is to transform the
 * virtual register state (including the stack pointer) such that
 * the call frame is unwound and the PC register points to the call site.
 */
static bool execute_personality_routine(unwind_state_t* state, byte_stream_t* stream, int pr_index) {
    size_t size;
    switch (pr_index) {
    case 0: // Personality routine #0, short frame, descriptors have 16-bit scope.
        size = 3;
        break;
    case 1: // Personality routine #1, long frame, descriptors have 16-bit scope.
    case 2: { // Personality routine #2, long frame, descriptors have 32-bit scope.
        uint8_t size_byte;
        if (!try_next_byte(stream, &size_byte)) {
            return false;
        }
        size = (uint32_t)size_byte * sizeof(uint32_t) + 2;
        break;
    }
    default: // Unknown personality routine.  Stop here.
        return false;
    }
    bool pc_was_set = false;
    while (size--) {
        uint8_t op;
        if (!try_next_byte(stream, &op)) {
            return false;
        }
        if ((op & 0xc0) == 0x00) {
            // "vsp = vsp + (xxxxxx << 2) + 4"
            set_reg(state, R_SP, state->gregs[R_SP] + ((op & 0x3f) << 2) + 4);
        } else if ((op & 0xc0) == 0x40) {
            // "vsp = vsp - (xxxxxx << 2) - 4"
            set_reg(state, R_SP, state->gregs[R_SP] - ((op & 0x3f) << 2) - 4);
        } else if ((op & 0xf0) == 0x80) {
            uint8_t op2;
            if (!(size--) || !try_next_byte(stream, &op2)) {
                return false;
            }
            uint32_t mask = (((uint32_t)op & 0x0f) << 12) | ((uint32_t)op2 << 4);
            if (mask) {
                // "Pop up to 12 integer registers under masks {r15-r12}, {r11-r4}"
                if (!try_pop_registers(state, mask)) {
                    return false;
                }
                if (mask & (1 << R_PC)) {
                    pc_was_set = true;
                }
            } else {
                // "Refuse to unwind"
                return false;
            }
        } else if ((op & 0xf0) == 0x90) {
            if (op != 0x9d && op != 0x9f) {
                // "Set vsp = r[nnnn]"
                set_reg(state, R_SP, state->gregs[op & 0x0f]);
            } else {
                // "Reserved as prefix for ARM register to register moves"
                // "Reserved as prefix for Intel Wireless MMX register to register moves"
                return false;
            }
        } else if ((op & 0xf8) == 0xa0) {
            // "Pop r4-r[4+nnn]"
            uint32_t mask = (0x0ff0 >> (7 - (op & 0x07))) & 0x0ff0;
            if (!try_pop_registers(state, mask)) {
                return false;
            }
        } else if ((op & 0xf8) == 0xa8) {
            // "Pop r4-r[4+nnn], r14"
            uint32_t mask = ((0x0ff0 >> (7 - (op & 0x07))) & 0x0ff0) | 0x4000;
            if (!try_pop_registers(state, mask)) {
                return false;
            }
        } else if (op == 0xb0) {
            // "Finish"
            break;
        } else if (op == 0xb1) {
            uint8_t op2;
            if (!(size--) || !try_next_byte(stream, &op2)) {
                return false;
            }
            if (op2 != 0x00 && (op2 & 0xf0) == 0x00) {
                // "Pop integer registers under mask {r3, r2, r1, r0}"
                if (!try_pop_registers(state, op2)) {
                    return false;
                }
            } else {
                // "Spare"
                return false;
            }
        } else if (op == 0xb2) {
            // "vsp = vsp + 0x204 + (uleb128 << 2)"
            uint32_t value = 0;
            uint32_t shift = 0;
            uint8_t op2;
            do {
                if (!(size--) || !try_next_byte(stream, &op2)) {
                    return false;
                }
                value |= (op2 & 0x7f) << shift;
                shift += 7;
            } while (op2 & 0x80);
            set_reg(state, R_SP, state->gregs[R_SP] + (value << 2) + 0x204);
        } else if (op == 0xb3) {
            // "Pop VFP double-precision registers D[ssss]-D[ssss+cccc] saved (as if) by FSTMFDX"
            uint8_t op2;
            if (!(size--) || !try_next_byte(stream, &op2)) {
                return false;
            }
            set_reg(state, R_SP, state->gregs[R_SP] + (uint32_t)(op2 & 0x0f) * 8 + 12);
        } else if ((op & 0xf8) == 0xb8) {
            // "Pop VFP double-precision registers D[8]-D[8+nnn] saved (as if) by FSTMFDX"
            set_reg(state, R_SP, state->gregs[R_SP] + (uint32_t)(op & 0x07) * 8 + 12);
        } else if ((op & 0xf8) == 0xc0) {
            // "Intel Wireless MMX pop wR[10]-wR[10+nnn]"
            set_reg(state, R_SP, state->gregs[R_SP] + (uint32_t)(op & 0x07) * 8 + 8);
        } else if (op == 0xc6) {
            // "Intel Wireless MMX pop wR[ssss]-wR[ssss+cccc]"
            uint8_t op2;
            if (!(size--) || !try_next_byte(stream, &op2)) {
                return false;
            }
            set_reg(state, R_SP, state->gregs[R_SP] + (uint32_t)(op2 & 0x0f) * 8 + 8);
        } else if (op == 0xc7) {
            uint8_t op2;
            if (!(size--) || !try_next_byte(stream, &op2)) {
                return false;
            }
            if (op2 != 0x00 && (op2 & 0xf0) == 0x00) {
                // "Intel Wireless MMX pop wCGR registers under mask {wCGR3,2,1,0}"
                set_reg(state, R_SP, state->gregs[R_SP] + __builtin_popcount(op2) * 4);
            } else {
                // "Spare"
                return false;
            }
        } else if (op == 0xc8) {
            // "Pop VFP double precision registers D[16+ssss]-D[16+ssss+cccc]
            // saved (as if) by FSTMFD"
            uint8_t op2;
            if (!(size--) || !try_next_byte(stream, &op2)) {
                return false;
            }
            set_reg(state, R_SP, state->gregs[R_SP] + (uint32_t)(op2 & 0x0f) * 8 + 8);
        } else if (op == 0xc9) {
            // "Pop VFP double precision registers D[ssss]-D[ssss+cccc] saved (as if) by FSTMFDD"
            uint8_t op2;
            if (!(size--) || !try_next_byte(stream, &op2)) {
                return false;
            }
            set_reg(state, R_SP, state->gregs[R_SP] + (uint32_t)(op2 & 0x0f) * 8 + 8);
        } else if ((op == 0xf8) == 0xd0) {
            // "Pop VFP double-precision registers D[8]-D[8+nnn] saved (as if) by FSTMFDD"
            set_reg(state, R_SP, state->gregs[R_SP] + (uint32_t)(op & 0x07) * 8 + 8);
        } else {
            // "Spare"
            return false;
        }
    }
    if (!pc_was_set) {
        set_reg(state, R_PC, state->gregs[R_LR]);
    }
    return true;
}

static uintptr_t rewind_pc_arch(uintptr_t pc) {
    if (pc & 1) {
        /* Thumb mode - need to check whether the bl(x) has long offset or not.
         * Examples:
         *
         * arm blx in the middle of thumb:
         * 187ae:       2300            movs    r3, #0
         * 187b0:       f7fe ee1c       blx     173ec
         * 187b4:       2c00            cmp     r4, #0
         *
         * arm bl in the middle of thumb:
         * 187d8:       1c20            adds    r0, r4, #0
         * 187da:       f136 fd15       bl      14f208
         * 187de:       2800            cmp     r0, #0
         *
         * pure thumb:
         * 18894:       189b            adds    r3, r3, r2
         * 18896:       4798            blx     r3
         * 18898:       b001            add     sp, #4
         */
        uint16_t prev1, prev2;
        if (try_get_half_word(pc - 5, &prev1)
            && ((prev1 & 0xf000) == 0xf000)
            && try_get_half_word(pc - 3, &prev2)
            && ((prev2 & 0xe000) == 0xe000)) {
            pc -= 4; // long offset
        } else {
            pc -= 2;
        }
    } else {
        /* ARM mode, all instructions are 32bit.  Yay! */
        pc -= 4;
    }
    return pc;
}

static backtrace_frame_t* add_backtrace_entry(uintptr_t pc, backtrace_frame_t* backtrace,
        size_t ignore_depth, size_t max_depth,
        size_t* ignored_frames, size_t* returned_frames) {
    if (*ignored_frames < ignore_depth) {
        *ignored_frames += 1;
        return NULL;
    }
    if (*returned_frames >= max_depth) {
        return NULL;
    }
    backtrace_frame_t* frame = &backtrace[*returned_frames];
    frame->absolute_pc = pc;
    frame->stack_top = 0;
    frame->stack_size = 0;
    *returned_frames += 1;
    return frame;
}

static ssize_t unwind_backtrace_common(unwind_state_t* state, backtrace_frame_t* backtrace, size_t ignore_depth, size_t max_depth) {
    size_t ignored_frames = 0;
    size_t returned_frames = 0;
    
	for (size_t index = 0; returned_frames < max_depth; index++) {
        uintptr_t pc = index ? rewind_pc_arch(state->gregs[R_PC]) : state->gregs[R_PC];
        backtrace_frame_t* frame = add_backtrace_entry(pc,
                backtrace, ignore_depth, max_depth, &ignored_frames, &returned_frames);
        if (frame) {
            frame->stack_top = state->gregs[R_SP];
        }
        
		uintptr_t handler = get_exception_handler(pc);
        if (!handler) {
            // If there is no handler for the PC and this is the first frame,
            // then the program may have branched to an invalid address.
            // Try starting from the LR instead, otherwise stop unwinding.
            if (index == 0 && state->gregs[R_LR]
                    && state->gregs[R_LR] != state->gregs[R_PC]) {
                set_reg(state, R_PC, state->gregs[R_LR]);
                continue;
            } else {
                break;
            }
        }
        
		byte_stream_t stream;
        stream.ptr = handler;
        uint8_t pr;
        if (!try_next_byte(&stream, &pr)) {
            break;
        }
        if ((pr & 0xf0) != 0x80) {
            // The first word is a place-relative pointer to a generic personality
            // routine function.  We don't support invoking such functions, so stop here.
            break;
        }
        // The first byte indicates the personality routine to execute.
        // Following bytes provide instructions to the personality routine.
        if (!execute_personality_routine(state, &stream, pr & 0x0f)) {
            break;
        }
        if (frame && state->gregs[R_SP] > frame->stack_top) {
            frame->stack_size = state->gregs[R_SP] - frame->stack_top;
        }
        if (!state->gregs[R_PC]) {
            break;
        }
    }
    // Ran out of frames that we could unwind using handlers.
    // Add a final entry for the LR if it looks sane and call it good.
    if (returned_frames < max_depth
            && state->gregs[R_LR]
            && state->gregs[R_LR] != state->gregs[R_PC]
            /*&& is_executable_map(map_info_list, state->gregs[R_LR])*/) {
        // We don't know where the stack for this extra frame starts so we
        // don't return any stack information for it.
        add_backtrace_entry(rewind_pc_arch(state->gregs[R_LR]),
                backtrace, ignore_depth, max_depth, &ignored_frames, &returned_frames);
    }
    return returned_frames;
}

ssize_t unwind_backtrace_signal_arch(void* sigcontext, backtrace_frame_t* backtrace, size_t ignore_depth, size_t max_depth) {
    const ucontext_t* uc = (const ucontext_t*)sigcontext;
    unwind_state_t state;
    state.gregs[0] = uc->uc_mcontext.arm_r0;
    state.gregs[1] = uc->uc_mcontext.arm_r1;
    state.gregs[2] = uc->uc_mcontext.arm_r2;
    state.gregs[3] = uc->uc_mcontext.arm_r3;
    state.gregs[4] = uc->uc_mcontext.arm_r4;
    state.gregs[5] = uc->uc_mcontext.arm_r5;
    state.gregs[6] = uc->uc_mcontext.arm_r6;
    state.gregs[7] = uc->uc_mcontext.arm_r7;
    state.gregs[8] = uc->uc_mcontext.arm_r8;
    state.gregs[9] = uc->uc_mcontext.arm_r9;
    state.gregs[10] = uc->uc_mcontext.arm_r10;
    state.gregs[11] = uc->uc_mcontext.arm_fp;
    state.gregs[12] = uc->uc_mcontext.arm_ip;
    state.gregs[13] = uc->uc_mcontext.arm_sp;
    state.gregs[14] = uc->uc_mcontext.arm_lr;
    state.gregs[15] = uc->uc_mcontext.arm_pc;

    return unwind_backtrace_common(&state, backtrace, ignore_depth, max_depth);
}

#endif //PLATFORM_ANDROID_ARM

int32 unwind_backtrace_signal(void* sigcontext, uint64* Backtrace, int32 MaxDepth)
{
	int32 NumFrames = 0;

#if PLATFORM_ANDROID_ARM
	const int32 BackTraceFramesSize = MaxDepth * sizeof(backtrace_frame_t);
	backtrace_frame_t* BackTraceFrames = static_cast<backtrace_frame_t*>(FMemory_Alloca(BackTraceFramesSize));
	FPlatformMemory::Memzero(BackTraceFrames, BackTraceFramesSize);

	NumFrames = unwind_backtrace_signal_arch(sigcontext, BackTraceFrames, 0, MaxDepth);
	for (int32 i = 0; i < NumFrames; ++i)
	{
		Backtrace[i] = BackTraceFrames[i].absolute_pc;
	}
#endif //PLATFORM_ANDROID_ARM

	return NumFrames;
}