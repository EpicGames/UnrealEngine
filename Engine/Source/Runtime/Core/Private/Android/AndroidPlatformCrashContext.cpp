// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AndroidPlatformCrashContext.cpp: implementations of Android platform crash context.
=============================================================================*/
#include "Android/AndroidPlatformCrashContext.h"
#include <sys/types.h>
#include <unistd.h>
#include <stdlib.h>
#include <limits.h>
#include <dlfcn.h>
#include "Containers/StringConv.h"
#include "CoreGlobals.h"
#include "Misc/App.h"
#include "Misc/Guid.h"
#include "HAL/PlatformStackWalk.h"
#include "Misc/Paths.h"
#include "GenericPlatform/GenericPlatformOutputDevices.h"
#include "HAL/FileManager.h"
#include "HAL/IConsoleManager.h"
#include "HAL/ThreadManager.h"
#include "HAL/RunnableThread.h"
#include "GenericPlatform/GenericPlatformCrashContext.h"
#include "Android/AndroidJniGameActivity.h"
#include "Async/TaskGraphInterfaces.h"

static int64 GetAndroidLibraryBaseAddress();

extern FString AndroidThunkCpp_GetMetaDataString(const FString& Key);

extern FString AndroidRelativeToAbsolutePath(bool bUseInternalBasePath, FString RelPath);

/** Java to native crash context k/v setting API  */
static void SetCrashContextOnGameThread(const FString KeyIn, const FString ValueIn)
{
	if (IsInGameThread())
	{
		FGenericCrashContext::SetGameData(KeyIn, ValueIn);
	}
	else if (FTaskGraphInterface::IsRunning())
	{
		FFunctionGraphTask::CreateAndDispatchWhenReady(
			[Key = KeyIn, Value = ValueIn]()
			{
				FGenericCrashContext::SetGameData(Key, Value);
			}
		, TStatId(), NULL, ENamedThreads::GameThread);
	}
	else
	{
		UE_LOGF(LogAndroid, Log, "Failed to set crash context `%ls` = '%ls'", *KeyIn, *ValueIn);
	}
}

#if USE_ANDROID_JNI
void JNICALL UE::Jni::FGameActivity::nativeCrashContextSetStringKey(JNIEnv* jenv, jobject thiz, jstring JavaKey, jstring JavaValue)
{
	FString Key, Value;
	Key = FJavaHelper::FStringFromParam(jenv, JavaKey);
	Value = FJavaHelper::FStringFromParam(jenv, JavaValue);

	SetCrashContextOnGameThread(Key, Value);
}

void JNICALL UE::Jni::FGameActivity::nativeCrashContextSetBooleanKey(JNIEnv* jenv, jobject thiz, jstring JavaKey, jboolean JavaValue)
{
	FString Key;
	Key = FJavaHelper::FStringFromParam(jenv, JavaKey);
	SetCrashContextOnGameThread(Key, JavaValue ? TEXT("true") : TEXT("false"));
}

void JNICALL UE::Jni::FGameActivity::nativeCrashContextSetIntegerKey(JNIEnv* jenv, jobject thiz, jstring JavaKey, jint JavaValue)
{
	FString Key;
	Key = FJavaHelper::FStringFromParam(jenv, JavaKey);
	SetCrashContextOnGameThread(Key, *TTypeToString<int32>::ToString(JavaValue));
}

void JNICALL UE::Jni::FGameActivity::nativeCrashContextSetFloatKey(JNIEnv* jenv, jobject thiz, jstring JavaKey, jfloat JavaValue)
{
	FString Key;
	Key = FJavaHelper::FStringFromParam(jenv, JavaKey);
	SetCrashContextOnGameThread(Key, *TTypeToString<float>::ToString(JavaValue));
}

void JNICALL UE::Jni::FGameActivity::nativeCrashContextSetDoubleKey(JNIEnv* jenv, jobject thiz, jstring JavaKey, jdouble JavaValue)
{
	FString Key;
	Key = FJavaHelper::FStringFromParam(jenv, JavaKey);
	SetCrashContextOnGameThread(Key, *TTypeToString<double>::ToString(JavaValue));
}

#endif
/** Implement platform specific static cleanup function */
void FGenericCrashContext::CleanupPlatformSpecificFiles()
{
}

struct FAndroidCrashInfo
{
	FAndroidCrashInfo()
	{
	}

	void Init()
	{
		if (!bInitialized)
		{
			FGuid RunGUID = FGuid::NewGuid();
			FCStringAnsi::Strcpy(AppName, TCHAR_TO_UTF8(FApp::GetProjectName()));
			FString LogPath = FGenericPlatformOutputDevices::GetAbsoluteLogFilename();
			LogPath = AndroidRelativeToAbsolutePath(false, LogPath);
			FCStringAnsi::Strcpy(AppLogPath, TCHAR_TO_UTF8(*LogPath));

			// Cache & create the crash report folder.
			FString ReportPath = FPaths::GameAgnosticSavedDir() / TEXT("Crashes");
			ReportPath = AndroidRelativeToAbsolutePath(true, ReportPath);
			IFileManager::Get().MakeDirectory(*ReportPath, true);
			FCStringAnsi::Strcpy(AndroidCrashReportPath, TCHAR_TO_UTF8(*ReportPath));

			FCStringAnsi::Strcpy(ProjectNameUTF8, TCHAR_TO_UTF8(FApp::GetProjectName()));
			FAndroidCrashContext::GenerateReportDirectoryName(TargetDirectory);
			bInitialized = true;
		}
	}

	static const int32 MaxAppNameSize = 128;
	char AppName[MaxAppNameSize] = { 0 };
	char AndroidCrashReportPath[FAndroidCrashContext::CrashReportMaxPathSize] = { 0 };
	char AppLogPath[FAndroidCrashContext::CrashReportMaxPathSize] = { 0 };
	char JavaLog[FAndroidCrashContext::CrashReportMaxPathSize] = { 0 };
	char TargetDirectory[FAndroidCrashContext::CrashReportMaxPathSize] = { 0 };
	char ProjectNameUTF8[FAndroidCrashContext::CrashReportMaxPathSize] = { 0 };
	bool bInitialized = false;
} GAndroidCrashInfo;

const FString FAndroidCrashContext::GetGlobalCrashDirectoryPath()
{
	return FString(GAndroidCrashInfo.TargetDirectory);
}

void FAndroidCrashContext::GetGlobalCrashDirectoryPath(char(&DirectoryNameOUT)[CrashReportMaxPathSize])
{
	FCStringAnsi::Strncpy(DirectoryNameOUT, GAndroidCrashInfo.TargetDirectory, CrashReportMaxPathSize);
}

const ANSICHAR* FAndroidCrashContext::ItoANSI(uint64 Val, uint64 Base, uint32 Len)
{
	static ANSICHAR InternalBuffer[64] = { 0 };

	uint64 i = 62;
	int32 pad = Len;
	Base = FMath::Clamp<uint64>(Base, 2, 16);
	if (Val)
	{
		for (; Val && i; --i, Val /= Base, --pad)
		{
			InternalBuffer[i] = "0123456789abcdef"[Val % Base];
		}
	}
	else
	{
		InternalBuffer[i--] = '0';
		--pad;
	}

	while (pad > 0)
	{
		InternalBuffer[i--] = '0';
		--pad;
	}

	return &InternalBuffer[i + 1];
}


void FAndroidCrashContext::GenerateReportDirectoryName(char(&DirectoryNameOUT)[CrashReportMaxPathSize])
{
	FGuid ReportGUID = FGuid::NewGuid();
	FCStringAnsi::Strncpy(DirectoryNameOUT, GAndroidCrashInfo.AndroidCrashReportPath, CrashReportMaxPathSize);
	FCStringAnsi::StrncatTruncateDest(DirectoryNameOUT, CrashReportMaxPathSize, "/CrashReport-UE-");
	FCStringAnsi::StrncatTruncateDest(DirectoryNameOUT, CrashReportMaxPathSize, GAndroidCrashInfo.ProjectNameUTF8);
	FCStringAnsi::StrncatTruncateDest(DirectoryNameOUT, CrashReportMaxPathSize, "-pid-");
	FCStringAnsi::StrncatTruncateDest(DirectoryNameOUT, CrashReportMaxPathSize, ItoANSI((uint32)getpid(), 10));
	FCStringAnsi::StrncatTruncateDest(DirectoryNameOUT, CrashReportMaxPathSize, "-");
	FCStringAnsi::StrncatTruncateDest(DirectoryNameOUT, CrashReportMaxPathSize, ItoANSI(ReportGUID.A, 16, 8));
	FCStringAnsi::StrncatTruncateDest(DirectoryNameOUT, CrashReportMaxPathSize, ItoANSI(ReportGUID.B, 16, 8));
	FCStringAnsi::StrncatTruncateDest(DirectoryNameOUT, CrashReportMaxPathSize, ItoANSI(ReportGUID.C, 16, 8));
	FCStringAnsi::StrncatTruncateDest(DirectoryNameOUT, CrashReportMaxPathSize, ItoANSI(ReportGUID.D, 16, 8));
}

static void CrashReportFileCopy(const char* DestPath, const char* SourcePath)
{
	int SourceHandle = open(SourcePath, O_RDONLY);
	int DestHandle = open(DestPath, O_CREAT | O_WRONLY, S_IRUSR | S_IWUSR);

	char Data[PATH_MAX] = {};
	int Bytes = 0;
	while ((Bytes = read(SourceHandle, Data, PATH_MAX)) > 0)
	{
		write(DestHandle, Data, Bytes);
	}

	close(DestHandle);
	close(SourceHandle);
}

void FAndroidCrashContext::StoreCrashInfo(bool bWriteLog) const
{
	char FilePath[CrashReportMaxPathSize] = { 0 };
	FCStringAnsi::Strncpy(FilePath, ReportDirectory, CrashReportMaxPathSize);
	FCStringAnsi::StrncatTruncateDest(FilePath, CrashReportMaxPathSize, "/");
	FCStringAnsi::StrncatTruncateDest(FilePath, CrashReportMaxPathSize, FGenericCrashContext::CrashContextRuntimeXMLNameA);
	SerializeAsXML(*FString(FilePath)); // CreateFileWriter will also create destination directory.

	if(bWriteLog)
	{
		// copy log:
		FCStringAnsi::Strncpy(FilePath, ReportDirectory, CrashReportMaxPathSize);
		FCStringAnsi::StrncatTruncateDest(FilePath, CrashReportMaxPathSize, "/");
		FCStringAnsi::StrncatTruncateDest(FilePath, CrashReportMaxPathSize, FCStringAnsi::Strlen(GAndroidCrashInfo.AppName) ? GAndroidCrashInfo.AppName : "UE4");
		FCStringAnsi::StrncatTruncateDest(FilePath, CrashReportMaxPathSize, ".log");
		CrashReportFileCopy(FilePath, GAndroidCrashInfo.AppLogPath);
	}
}


//Create a separate file containing thread context info (callstacks etc) in xml form
// This is added to the crash report xml at during pre-processing time.
void FAndroidCrashContext::DumpAllThreadCallstacks(FAsyncThreadBackTrace* BackTrace, int NumThreads) const
{
	if (NumThreads == 0)
	{
		return;
	}

	char FilePath[FAndroidCrashContext::CrashReportMaxPathSize] = { 0 };
	FCStringAnsi::Strncpy(FilePath, ReportDirectory, FAndroidCrashContext::CrashReportMaxPathSize);
	FCStringAnsi::StrncatTruncateDest(FilePath, FAndroidCrashContext::CrashReportMaxPathSize, "/AllThreads.txt");
	TArray<FCrashStackFrame> CrashStackFrames;
	CrashStackFrames.Empty(32);
	int DestHandle = open(FilePath, O_CREAT | O_WRONLY, S_IRUSR | S_IWUSR);
	if (DestHandle >= 0)
	{
		int32 CurrentThreadID = FPlatformTLS::GetCurrentThreadId();
		auto Write = [](int FileHandle, const ANSICHAR* Buffer)
		{
			write(FileHandle, Buffer, FCStringAnsi::Strlen(Buffer));
		};
		auto Writeln = [](int FileHandle, const ANSICHAR* Buffer)
		{
			write(FileHandle, Buffer, FCStringAnsi::Strlen(Buffer));
			write(FileHandle, "\n", 1);
		};

		auto CaptureCallstack = [CurrentThreadID](FAsyncThreadBackTrace* BackTrace)
		{
			if (CurrentThreadID == BackTrace->ThreadID)
			{
				BackTrace->Depth = FPlatformStackWalk::CaptureStackBackTrace(BackTrace->BackTrace, BackTrace->StackTraceMaxDepth);
				BackTrace->Flag.store(1, std::memory_order_release);
			}
			else
			{
				FPlatformStackWalk::CaptureThreadStackBackTraceAsync(BackTrace);
			}
		};

		auto WriteThreadEntry = [Writeln, Write, this, DestHandle, &CrashStackFrames](FAsyncThreadBackTrace* BackTrace)
		{
			if (BackTrace->Depth)
			{
				ANSICHAR Line[256];
				Writeln(DestHandle, "<Thread>");
				Write(DestHandle, "<CallStack>");
				// Write stack
				GetPortableCallStack(BackTrace->BackTrace, BackTrace->Depth, CrashStackFrames);
				for (const FCrashStackFrame& CrashStackFrame : CrashStackFrames)
				{
					FCStringAnsi::Strncpy(Line, TCHAR_TO_UTF8(*CrashStackFrame.ModuleName), UE_ARRAY_COUNT(Line));
					FCStringAnsi::StrncatTruncateDest(Line, UE_ARRAY_COUNT(Line), " 0x");
					FCStringAnsi::StrncatTruncateDest(Line, UE_ARRAY_COUNT(Line), ItoANSI(CrashStackFrame.BaseAddress, (uint64)16, 16));
					FCStringAnsi::StrncatTruncateDest(Line, UE_ARRAY_COUNT(Line), " + ");
					FCStringAnsi::StrncatTruncateDest(Line, UE_ARRAY_COUNT(Line), ItoANSI(CrashStackFrame.Offset, (uint64)16, 16));
					Writeln(DestHandle, Line);
				}
				Writeln(DestHandle, "</CallStack>");
				Writeln(DestHandle, "<IsCrashed>false</IsCrashed>");
				Writeln(DestHandle, "<Registers/>");
				// write Thread Id
				FCStringAnsi::Strncpy(Line, ItoANSI((uint64)BackTrace->ThreadID, (uint64)10), UE_ARRAY_COUNT(Line));
				Write(DestHandle, "<ThreadID>");
				Write(DestHandle, Line);
				Writeln(DestHandle, "</ThreadID>");

				// write Thread Name
				Write(DestHandle, "<ThreadName>");
				FMemory::Memcpy(Line, BackTrace->ThreadName, FAsyncThreadBackTrace::MaxThreadName);
				Line[FAsyncThreadBackTrace::MaxThreadName] = 0;
				Write(DestHandle, Line);
				Writeln(DestHandle, "</ThreadName>");

				Writeln(DestHandle, "</Thread>");
			}
		};

		Writeln(DestHandle, "<Threads>");

		uint32 CaptureCallstacks = 0;
		uint32 CallstacksRecorded = 0;
		for (int i = 0; i < NumThreads; ++i)
		{
			BackTrace[i].Flag.store(0, std::memory_order_relaxed);
			BackTrace[i].Depth = 0;
		}

		// On android the Game thread is that which calls android_main entry point. 
		// Explicitly call it here as the thread manager is not aware of it.
		if (CrashingThreadId != GGameThreadId)
		{
			BackTrace[0].ThreadID = GGameThreadId;
			FCStringAnsi::Strcpy(BackTrace[0].ThreadName, "GameThread");
			CaptureCallstack(&BackTrace[0]);
			++CaptureCallstacks;
		}

		FThreadManager::Get().ForEachThread([CaptureCallstack, BackTrace, &CaptureCallstacks, &NumThreads, &CrashingThreadId = CrashingThreadId](uint32 ThreadID, FRunnableThread* Runnable)
		{
			if ((CaptureCallstacks < NumThreads) && (CrashingThreadId != ThreadID))
			{
				FAsyncThreadBackTrace* Trace = &BackTrace[CaptureCallstacks];
				Trace->ThreadID = ThreadID;
				const FString& ThreadName = Runnable->GetThreadName();
				FMemory::Memcpy(Trace->ThreadName, TCHAR_TO_UTF8(*ThreadName), FMath::Min(ThreadName.Len() + 1, FAsyncThreadBackTrace::MaxThreadName));

				CaptureCallstack(Trace);
				++CaptureCallstacks;
			}
		});

		const float PollTime = 0.001f;
		extern float GThreadCallStackMaxWait;

		for (float CurrentTime = 0; CurrentTime <= GThreadCallStackMaxWait; CurrentTime += PollTime)
		{
			CallstacksRecorded = 0;
			for (int i = 0; i < CaptureCallstacks; ++i)
			{
				CallstacksRecorded += BackTrace[i].Flag.load(std::memory_order_acquire);
			}
			if (CallstacksRecorded == CaptureCallstacks)
			{
				break;
			}
			FPlatformProcess::SleepNoStats(PollTime);
		}

		for (int i = 0; i < CaptureCallstacks; ++i)
		{
			WriteThreadEntry(&BackTrace[i]);
		}

		Writeln(DestHandle, "</Threads>");
		close(DestHandle);

		if (CallstacksRecorded == 0)
		{
			unlink(FilePath); // remove the file if nothing was written.
		}
	}
}

void FAndroidCrashContext::Initialize()
{
	GAndroidCrashInfo.Init();
}

FAndroidCrashContext::FAndroidCrashContext(ECrashContextType InType, const TCHAR* InErrorMessage)
:	FGenericCrashContext(InType, InErrorMessage)
, Signal(0)
, Info(NULL)
, Context(NULL)
{
	switch(GetType())
	{
		case ECrashContextType::AbnormalShutdown:
		case ECrashContextType::Ensure:
			// create a new report folder.
			GenerateReportDirectoryName(ReportDirectory);
			break;
		default:
			GetGlobalCrashDirectoryPath(ReportDirectory);
			break;
	}
}


void FAndroidCrashContext::SetOverrideCallstack(const FString& OverrideCallstackIN)
{
	OverrideCallstack.Reset();
	TArray<FString> OutArray;
	OverrideCallstackIN.ParseIntoArrayLines(OutArray);

	for (const FString& Line : OutArray)
	{
		AppendEscapedXMLString(OverrideCallstack, *Line);
		OverrideCallstack += TEXT("&#xA;");
		OverrideCallstack += LINE_TERMINATOR;
	}
}

const TCHAR* FAndroidCrashContext::GetCallstackProperty() const
{
	return *OverrideCallstack;
}

void FAndroidCrashContext::CaptureCrashInfo()
{
	CapturePortableCallStack(nullptr, Context);
}

static int64 GetAndroidLibraryBaseAddress()
{
	int64 BaseAddress = 0;

	const char *LibraryName = "libUnreal.so";
	int32 LibraryLength = strlen(LibraryName);

	// try to open process map file
	FILE *file = fopen("/proc/self/maps", "r");
	if (file == NULL)
	{
		return BaseAddress;
	}

	char LineBuffer[256];
	LineBuffer[255] = 0;
	while (fgets(LineBuffer, 255, file) != NULL)
	{
		int32 BufferLength = strlen(LineBuffer);
		if (BufferLength > 0 && LineBuffer[BufferLength - 1] == '\n')
		{
			BufferLength--;
			LineBuffer[BufferLength] = 0;
		}

		// does it end with library name?
		if (BufferLength < LibraryLength || memcmp(LineBuffer + BufferLength - LibraryLength, LibraryName, LibraryLength))
		{
			continue;
		}

		// parse the line
		int64 StartAddress, EndAddress, Offset;
		char flags[4];
		if (sscanf(LineBuffer, "%llx-%llx %c%c%c%c %llx", &StartAddress, &EndAddress, &flags[0], &flags[1], &flags[2], &flags[3], &Offset) != 7)
		{
			continue;
		}

		// needs to be r-x (ignore 4th)
		if (flags[0] == 'r' && flags[1] == '-' && flags[2] == 'x')
		{
			BaseAddress = StartAddress - Offset;
			break;
		}
	}
	fclose(file);
	return BaseAddress;
}

void FAndroidCrashContext::AddAndroidCrashProperty(const FString& Key, const FString& Value)
{
	AdditionalProperties.Add(Key, Value);
}

void FAndroidCrashContext::AddPlatformSpecificProperties() const
{
	for (TMap<FString, FString>::TConstIterator Iter(AdditionalProperties); Iter; ++Iter)
	{
		AddCrashProperty(*Iter.Key(), *Iter.Value());
	}
}

bool FAndroidCrashContext::ShouldIncludeInPortableCallStackHash(const FCrashStackFrame& Frame) const
{
	// Match against libUnreal since GetPortableCallStack puts the .so name in the callstack
	return Frame.ModuleName == TEXT("libUnreal");
}

void FAndroidCrashContext::GetPortableCallStack(const uint64* StackFrames, int32 NumStackFrames, TArray<FCrashStackFrame>& OutCallStack) const
{
	// Update the callstack with offsets from each module
	OutCallStack.Reset(NumStackFrames);
	for (int32 Idx = 0; Idx < NumStackFrames; Idx++)
	{
		const uint64 StackFrame = StackFrames[Idx];

		// Try to find the module containing this stack frame
		const FStackWalkModuleInfo* FoundModule = nullptr;

		Dl_info DylibInfo;
		int32 Result = dladdr((const void*)StackFrame, &DylibInfo);
		if (Result != 0)
		{
			ANSICHAR* DylibPath = (ANSICHAR*)DylibInfo.dli_fname;
			ANSICHAR* DylibName = FCStringAnsi::Strrchr(DylibPath, '/');
			if (DylibName)
			{
				DylibName += 1;
			}
			else
			{
				DylibName = DylibPath;
			}
			OutCallStack.Add(FCrashStackFrame(FPaths::GetBaseFilename(DylibName), (uint64)DylibInfo.dli_fbase, StackFrame - (uint64)DylibInfo.dli_fbase));
		}
		else
		{
			OutCallStack.Add(FCrashStackFrame(TEXT("Unknown"), 0, StackFrame));
		}
	}
}
