// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AndroidFile.cpp: Android platform implementations of File functions
=============================================================================*/

#include "Android/AndroidPlatformFile.h"

#if USE_ANDROID_FILE
#include "Misc/App.h"
#include "Misc/Paths.h"
#include "Android/AndroidJniGameActivity.h"
#include "HAL/IConsoleManager.h"

#include <dirent.h>
#include <unistd.h>
#include <android/asset_manager.h>
#include <android/asset_manager_jni.h>
#include <android/storage_manager.h>
#include <android/fdsan.h>
#include "Android/AndroidJava.h"
#include "Containers/Map.h"
#include "ProfilingDebugging/PlatformFileTrace.h"
#include "HAL/DiskUtilizationTracker.h"
#include <limits>
#include <sys/mman.h>
#include "Misc/ScopeLock.h"
#include "Misc/ScopeExit.h"
#include <Async/MappedFileHandle.h>

#include <atomic>

#if PLATFORM_USE_PLATFORM_FILE_MANAGED_STORAGE_WRAPPER
#include "HAL/IPlatformFileManagedStorageWrapper.h"
#endif //PLATFORM_USE_PLATFORM_FILE_MANAGED_STORAGE_WRAPPER

DEFINE_LOG_CATEGORY_STATIC(LogAndroidFile, Log, All);

#define LOG_ANDROID_FILE 0

#define LOG_ANDROID_FILE_MANIFEST 0

/** When ANDROID_DISALLOW_LOCAL_FILESYSTEM is true, FAndroidPlatformFile will only look in mounted OBB files, making it more difficult for cheaters to patch files. */
#ifndef ANDROID_DISALLOW_LOCAL_FILESYSTEM
#define ANDROID_DISALLOW_LOCAL_FILESYSTEM 0
#endif

// Support 64 bit file access.
//#define UE_ANDROID_FILE_64 0
#define UE_ANDROID_FILE_64 1

#if UE_ANDROID_FILE_64
	#define __lseek(_fd, _offset, _whence)			lseek64(_fd, _offset, _whence)
	#define __pread(_fd, _buf, _count, _offset)		pread64(_fd, _buf, _count, _offset)
	#define __pwrite(_fd, _buf, _count, _offset)	pwrite64(_fd, _buf, _count, _offset)
	#define __ftruncate(_fd, _length)				ftruncate64(_fd, _length)
#else
	#define __lseek(_fd, _offset, _whence)			lseek(_fd, _offset, _whence)
	#define __pread(_fd, _buf, _count, _offset)		pread(_fd, _buf, _count, _offset)
	#define __pwrite(_fd, _buf, _count, _offset)	pwrite(_fd, _buf, _count, _offset)
	#define __ftruncate(_fd, _length)				ftruncate(_fd, _length)
#endif

// make an FTimeSpan object that represents the "epoch" for time_t (from a stat struct)
const FDateTime AndroidEpoch(1970, 1, 1);

//
// FDSan has been added to Android in API level 29. However, symbols are weak, so it is possible 
// to determine the availability in runtime. NDK headers have them under API level define, so
// we duplicate those here if needed.
//
#if __ANDROID_API__ < 29
extern "C"
{
	void android_fdsan_exchange_owner_tag(int fd, uint64_t expected_tag, uint64_t new_tag) __attribute__((__weak__));
	int android_fdsan_close_with_tag(int fd, uint64_t tag) __attribute__((__weak__));
}
#endif // __ANDROID_API__ < 29

namespace UE::Android::Private
{
	struct FdSanConfig
	{
		FdSanConfig()
		{
			bIsEnabled = !FPlatformMisc::GetConfigRulesVariableAsUtf8("UseFdSan").Equals("false", ESearchCase::IgnoreCase);
		}

		bool bIsEnabled;
	};

	static bool ShouldUseFdSan()
	{
		static FdSanConfig Config;
		return Config.bIsEnabled;
	}
} // UE::Android::Private

namespace
{
	FFileStatData AndroidStatToUEFileData(struct stat& FileInfo)
	{
		const bool bIsDirectory = S_ISDIR(FileInfo.st_mode);

		int64 FileSize = -1;
		if (!bIsDirectory)
		{
			FileSize = FileInfo.st_size;
		}

		return FFileStatData(
			AndroidEpoch + FTimespan::FromSeconds((double)FileInfo.st_ctime),
			AndroidEpoch + FTimespan::FromSeconds((double)FileInfo.st_atime),
			AndroidEpoch + FTimespan::FromSeconds((double)FileInfo.st_mtime), 
			FileSize,
			bIsDirectory,
			!(FileInfo.st_mode & S_IWUSR)
		);
	}

	class FAndroidUniqueFd final
	{
	public:
		FAndroidUniqueFd() = default;

		explicit FAndroidUniqueFd(int32 InFd)
		{
			Reset(InFd);
		}

		FAndroidUniqueFd(const FAndroidUniqueFd&) = delete;
		FAndroidUniqueFd& operator=(const FAndroidUniqueFd&) = delete;

		FAndroidUniqueFd(FAndroidUniqueFd&& Other) noexcept
		{
			*this = MoveTemp(Other);
		}

		FAndroidUniqueFd& operator=(FAndroidUniqueFd&& Other) noexcept
		{
			if (this == &Other)
			{
				return *this;
			}

			Reset();

			if (Other.Fd != -1)
			{
				Fd       = Other.Fd;
				Other.Fd = -1;

				ExchangeTag(Fd, Other.GetTag(), GetTag());
			}

			return *this;
		}

		~FAndroidUniqueFd()
		{
			Reset();
		}

		FORCEINLINE int32 Get() const
		{
			return Fd;
		}

		FORCEINLINE bool IsValid() const
		{
			return Fd != -1;
		}

	private:
		int32 Fd = -1;

		void Reset(int32 NewFd = -1)
		{
			uint64 Tag = GetTag();

			if (Fd != -1)
			{
				Close(Fd, Tag);
				Fd = -1;
			}

			if (NewFd != -1)
			{
				Fd = NewFd;
				ExchangeTag(Fd, 0, Tag);
			}
		}

		uint64 GetTag() const
		{
			static_assert(sizeof(uint64) == sizeof(void*));
			return reinterpret_cast<uint64>(this);
		}

		static int Close(int32 Fd, uint64 Tag)
		{
			if (android_fdsan_close_with_tag && UE::Android::Private::ShouldUseFdSan())
			{
				return android_fdsan_close_with_tag(Fd, Tag);
			}
			return close(Fd);
		}

		static void ExchangeTag(int32 Fd, uint64 OldTag, uint64 NewTag)
		{
			if (android_fdsan_exchange_owner_tag && UE::Android::Private::ShouldUseFdSan())
			{
				android_fdsan_exchange_owner_tag(Fd, OldTag, NewTag);
			}
		}
	};
}

#define USE_UTIME 0


// AndroidProcess uses this for executable name
FString GAndroidProjectName;
FString GPackageName;
int32 GAndroidPackageVersion = 0;
int32 GAndroidPackagePatchVersion = 0;
FString GAndroidAppType;

#define ANDROID_MAX_OVERFLOW_FILES	32

// External File Path base - setup during load
FString GFilePathBase;
// Obb File Path base - setup during load
FString GOBBFilePathBase;
// Obb Main filepath
FString GOBBMainFilePath;
// Obb Patch filepath
FString GOBBPatchFilePath;
// Obb Overflow1 filepath
FString GOBBOverflow1FilePath;
// Obb Overflow2 filepath
FString GOBBOverflow2FilePath;
// Internal File Direcory Path (for application) - setup during load
FString GInternalFilePath;
// External File Direcory Path (for application) - setup during load
FString GExternalFilePath;
// External font path base - setup during load
FString GFontPathBase;

// Last opened OBB comment (set during mounting of OBB)
FString GLastOBBComment;

// Is the OBB in an APK file or not
bool GOBBinAPK;
FString GAPKFilename;

// Directory for log file on Android
bool GOverrideAndroidLogDir = false;
static FString AndroidLogDir;

#define FILEBASE_DIRECTORY "/UnrealGame/"

extern jobject AndroidJNI_GetJavaAssetManager();
extern AAssetManager * AndroidThunkCpp_GetAssetManager();

#if USE_ANDROID_JNI
//This function is declared in the Java-defined class, GameActivity.java: "public native void nativeSetObbInfo(String PackageName, int Version, int PatchVersion);"
void JNICALL UE::Jni::FGameActivity::nativeSetObbInfo(JNIEnv* jenv, jobject thiz, jstring ProjectName, jstring PackageName, jint Version, jint PatchVersion, jstring AppType)
{
	GAndroidProjectName = FJavaHelper::FStringFromParam(jenv, ProjectName);
	GPackageName = FJavaHelper::FStringFromParam(jenv, PackageName);
	GAndroidAppType = FJavaHelper::FStringFromParam(jenv, AppType);

	GAndroidPackageVersion = Version;
	GAndroidPackagePatchVersion = PatchVersion;
}
#if USE_ANDROID_STANDALONE

//This function is declared in the Java-defined class, AsisGameActivity.java: "public native String nativeGetObbComment();"
jstring JNICALL UE::Jni::Asis::FAsisGameActivity::nativeGetObbComment(JNIEnv* jenv, jclass clazz)
{
	return jenv->NewStringUTF(TCHAR_TO_UTF8(*GLastOBBComment));
}

#endif
#endif
// Constructs the base path for any files which are not in OBB/pak data
const FString &GetFileBasePath()
{
	static FString BasePath = GFilePathBase + FString(FILEBASE_DIRECTORY) + FApp::GetProjectName() + FString("/");
	return BasePath;
}


FString AndroidRelativeToAbsolutePath(bool bUseInternalBasePath, FString RelPath)
{
	if (RelPath.StartsWith(TEXT("../"), ESearchCase::CaseSensitive))
	{
		
		do {
			RelPath.RightChopInline(3, EAllowShrinking::No);
		} while (RelPath.StartsWith(TEXT("../"), ESearchCase::CaseSensitive));

		return (bUseInternalBasePath ? GInternalFilePath : GetFileBasePath()) / RelPath;
	}
	return RelPath;
}

/**
 * Wrapper around AAssetManager_openDir that returns a valid pointer only if the app's Assets actually contains the given "directory".
 *
 * AAssetManager_openDir always returns non-nullptr, even if there is no "directory", but it's tested anyways for extra stability.  Android Assets
 * underlying storage implementation data structure is flat, not hierarchical, so "directories" are imaginary.  Thus, Android asset storage doesn't
 * support empty directories, and we should not treat non-existent asset prefixes as stat-able directories because it breaks systems (like SQLite VFS)
 * that check for the existence of certain paths without any means of discriminating between files and directories.
 */
AAssetDir* OpenExistingAssetManagerDirectory(AAssetManager* AssetManager, const char* AssetPath)
{
	AAssetDir* Directory = AAssetManager_openDir(AssetManager, AssetPath);
	if (UNLIKELY(nullptr == Directory))
	{
		return nullptr;
	}

	const char* const AnyFileName = AAssetDir_getNextFileName(Directory);
	if (nullptr == AnyFileName)
	{
		AAssetDir_close(Directory);
		Directory = nullptr;
	}

	return Directory;
}

/**
	Android file handle implementation for partial (i.e. parcels) files.
	This can handle all the variations of accessing data for assets and files.
**/
class CORE_API FFileHandleAndroid final : public IFileHandle
{
	enum { READWRITE_SIZE = 1024 * 1024 };

	void LogInfo()
	{
#if LOG_ANDROID_FILE
		FPlatformMisc::LowLevelOutputDebugStringf(
			TEXT("FFileHandleAndroid => Asset = %p, Handle = %d, Bounds = [%d,%d)"),
			File->Asset, File->Handle.Get(), int32(Start), int32(Start+Length));
#endif
	}

public:

	struct FileReference
	{
		FString Path;
		AAsset * Asset;
		FAndroidUniqueFd Handle;

		FileReference() : Asset(nullptr)
		{
		}

		FileReference(const FString & path, AAsset * asset)
			: Path(path), Asset(asset)
		{
		}

		FileReference(const FString & path, FAndroidUniqueFd handle)
			: Path(path), Asset(nullptr), Handle(MoveTemp(handle))
		{
		}

		~FileReference()
		{
			if (nullptr != Asset)
			{
				AAsset_close(Asset);
			}
		}
	};

	TSharedPtr<FileReference> File;
	int64 Start;
	int64 Length;
	int64 CurrentOffset;

	FORCEINLINE void CheckValid() const
	{
		check(File.IsValid() && File->Handle.IsValid());
	}

	// Invalid handle.
	FFileHandleAndroid()
		: File(MakeShareable(new FileReference()))
		, Start(0), Length(0), CurrentOffset(0)
	{
	}

	// Handle that covers a subsegment of another handle.
	FFileHandleAndroid(const FFileHandleAndroid & base,
		int64 start, int64 length)
		: File(base.File)
		, Start(base.Start + start), Length(length)
		, CurrentOffset(base.Start + start)
	{
		CheckValid();
		LogInfo();
	}

	// Handle that covers a subsegment of provided file.
	FFileHandleAndroid(const FString & path, FAndroidUniqueFd filehandle, int64 filestart, int64 filelength)
		: File(MakeShareable(new FileReference(path, MoveTemp(filehandle))))
		, Start(filestart), Length(filelength), CurrentOffset(0)
	{
		CheckValid();
#if UE_ANDROID_FILE_64
		lseek64(File->Handle.Get(), filestart, SEEK_SET);
#else
		lseek(File->Handle.Get(), filestart, SEEK_SET);
#endif
		LogInfo();
	}

	// Handle that covers the entire file content.
	FFileHandleAndroid(const FString & path, FAndroidUniqueFd filehandle)
		: File(MakeShareable(new FileReference(path, MoveTemp(filehandle))))
		, Start(0), Length(0), CurrentOffset(0)
	{
		CheckValid();
		Length = __lseek(File->Handle.Get(), 0, SEEK_END);
		__lseek(File->Handle.Get(), 0, SEEK_SET);
		LogInfo();
	}

	// Handle that covers the entire content of an asset.
	FFileHandleAndroid(const FString & path, AAsset * asset)
		: File(MakeShareable(new FileReference(path, asset)))
		, Start(0), Length(0), CurrentOffset(0)
	{
		off64_t OutStart = Start;
		off64_t OutLength = Length;
		File->Handle = FAndroidUniqueFd { AAsset_openFileDescriptor64(File->Asset, &OutStart, &OutLength) };
		Start = OutStart;
		Length = OutLength;
		CurrentOffset = Start;
		CheckValid();
		LogInfo();
	}

	virtual ~FFileHandleAndroid()
	{
	}

	virtual int64 Tell() override
	{
		CheckValid();
		int64 pos = CurrentOffset;
		check(pos != -1);
		return pos - Start; // We are treating 'tell' as a virtual location from file Start
	}

	virtual bool Seek(int64 NewPosition) override
	{
		CheckValid();
		// we need to offset all positions by the Start offset
		CurrentOffset = NewPosition += Start;
		check(NewPosition >= 0);
		
		return true;
	}

    virtual bool SeekFromEnd(int64 NewPositionRelativeToEnd = 0) override
	{
		CheckValid();
		check(NewPositionRelativeToEnd <= 0);
		// We need to convert this to a virtual offset inside the file we are interested in
		CurrentOffset = Start + (Length - NewPositionRelativeToEnd);
		return true;
	}

	

	virtual bool Read(uint8* Destination, int64 BytesToRead) override
	{
		CheckValid();
#if LOG_ANDROID_FILE
		FPlatformMisc::LowLevelOutputDebugStringf(
			TEXT("(%d/%d) FFileHandleAndroid:Read => Path = %s, BytesToRead = %d"),
			FAndroidTLS::GetCurrentThreadId(), File->Handle.Get(),
			*(File->Path), int32(BytesToRead));
#endif
		check(BytesToRead >= 0);
		check(Destination);
		bool bAtEndOfFile = CurrentOffset >= Start + Length;
		if (bAtEndOfFile)
		{
			// Fail if we tried reading when already at the end of the file.
			return false;
		}

		int64 TotalBytesRead = 0;
		FScopedDiskUtilizationTracker Tracker(BytesToRead, CurrentOffset);
		TRACE_PLATFORMFILE_BEGIN_READ(this, File->Handle.Get(), CurrentOffset, BytesToRead);
		ON_SCOPE_EXIT
		{
			TRACE_PLATFORMFILE_END_READ(this, TotalBytesRead);
		};

		do
		{
			int64 ThisSize = FMath::Min<int64>(READWRITE_SIZE, BytesToRead);
			
			ThisSize = __pread(File->Handle.Get(), Destination, ThisSize, CurrentOffset);
#if LOG_ANDROID_FILE
			FPlatformMisc::LowLevelOutputDebugStringf(
				TEXT("(%d/%d) FFileHandleAndroid:Read => Path = %s, ThisSize = %d, destination = %X"),
				FAndroidTLS::GetCurrentThreadId(), File->Handle.Get(),
				*(File->Path), int32(ThisSize), Destination);
#endif
			if (ThisSize < 0)
			{
				if (errno == EINTR)
				{
					// interrupted by signal, no error
					continue;
				}
				return false;
			}
			else if (ThisSize == 0)
			{
				// 0 is EOF
				bAtEndOfFile = true;
				break;
			}
			TotalBytesRead += ThisSize;
			CurrentOffset += ThisSize;
			Destination += ThisSize;
			BytesToRead -= ThisSize;
			bAtEndOfFile = CurrentOffset >= Start + Length;
		} while (BytesToRead > 0 && !bAtEndOfFile);

		return BytesToRead == 0 || bAtEndOfFile;
	}

	virtual bool ReadAt(uint8* Destination, int64 BytesToRead, int64 Offset) override
	{
		int64 TrueOffset = Start + Offset;
		return ReadInternal(Destination, BytesToRead, TrueOffset);
	}

	virtual bool Write(const uint8* Source, int64 BytesToWrite) override
	{
		CheckValid();
		if (nullptr != File->Asset)
		{
			// Can't write to assets.
			return false;
		}

		bool bSuccess = true;
		while (BytesToWrite > 0)
		{
			int64 ThisSize = FMath::Min<int64>(READWRITE_SIZE, BytesToWrite);
			check(Source);
			errno = EINTR;
			int64 Result = __pwrite(File->Handle.Get(), Source, ThisSize, CurrentOffset);
			if (Result <= 0)
			{
				if (errno == EINTR)
				{
					// interrupted by signal, no error
					continue;
				}
#if LOG_ANDROID_FILE
				int32 SaveErrno = errno;
				FPlatformMisc::LowLevelOutputDebugStringf(
					TEXT("(%d/%d) FFileHandleAndroid:Write => Path = %s, this size = %d, CurrentOffset = %d, Source = %p, Result = %d, errno = %d"),
					FAndroidTLS::GetCurrentThreadId(), File->Handle.Get(),
					*(File->Path), int32(ThisSize), CurrentOffset, Source, int32(Result), SaveErrno);
#endif
				bSuccess = false;
				break;
			}
#if LOG_ANDROID_FILE
			FPlatformMisc::LowLevelOutputDebugStringf(
				TEXT("(%d/%d) FFileHandleAndroid:Write => Path = %s, this size = %d, CurrentOffset = %d, Source = %p, Result = %d"),
				FAndroidTLS::GetCurrentThreadId(), File->Handle.Get(),
				*(File->Path), int32(ThisSize), CurrentOffset, Source, int32(Result));
#endif
			CurrentOffset += Result;
			Source += Result;
			BytesToWrite -= Result;
		}
		
		// Update the cached file length
		Length = FMath::Max(Length, CurrentOffset);
#if LOG_ANDROID_FILE
		FPlatformMisc::LowLevelOutputDebugStringf(
			TEXT("(%d/%d) FFileHandleAndroid:Write => Path = %s, final size %d"),
			FAndroidTLS::GetCurrentThreadId(), File->Handle.Get(),
			*(File->Path), Length);
#endif
		return bSuccess;
	}

	virtual bool Flush(const bool bFullFlush = false) override
	{
		CheckValid();
		if (nullptr != File->Asset)
		{
			// Can't write to assets.
			return false;
		}

		return bFullFlush
			? fsync(File->Handle.Get()) == 0
			: fdatasync(File->Handle.Get()) == 0;
	}

	virtual bool Truncate(int64 NewSize) override
	{
		CheckValid();
		if (nullptr != File->Asset)
		{
			// Can't write to assets.
			return false;
		}

		int Result = 0;
		do { Result = __ftruncate(File->Handle.Get(), NewSize); } while (Result < 0 && errno == EINTR);
		if (Result == 0)
		{
			const int CurrentPos = __lseek(File->Handle.Get(), 0, SEEK_CUR); 
			Length = __lseek(File->Handle.Get(), 0, SEEK_END);
			__lseek(File->Handle.Get(), CurrentPos, SEEK_SET);
		}
		return Result == 0;
	}

	virtual int64 Size() override
	{
		return Length;
	}

private:
	bool ReadInternal(uint8* Destination, int64 BytesToRead, int64 Offset)
	{
		CheckValid();
#if LOG_ANDROID_FILE
		FPlatformMisc::LowLevelOutputDebugStringf(
			TEXT("(%d/%d) FFileHandleAndroid:Read => Path = %s, BytesToRead = %d"),
			FAndroidTLS::GetCurrentThreadId(), File->Handle.Get(),
			*(File->Path), int32(BytesToRead));
#endif
		if (BytesToRead < 0 || Offset - Start < 0 || (BytesToRead + Offset - Start) > Size())
		{
			return false;
		}

		if (BytesToRead == 0)
		{
			return true;
		}

		check(Destination);

		int64 TotalBytesRead = 0;
		FScopedDiskUtilizationTracker Tracker(BytesToRead, Offset);
		TRACE_PLATFORMFILE_BEGIN_READ(this, File->Handle.Get(), Offset, BytesToRead);
		ON_SCOPE_EXIT
		{
			TRACE_PLATFORMFILE_END_READ(this, TotalBytesRead);
		};

		while (BytesToRead > 0)
		{
			int64 ThisSize = FMath::Min<int64>(READWRITE_SIZE, BytesToRead);

			ThisSize = __pread(File->Handle.Get(), Destination, ThisSize, Offset);
#if LOG_ANDROID_FILE
			FPlatformMisc::LowLevelOutputDebugStringf(
				TEXT("(%d/%d) FFileHandleAndroid:Read => Path = %s, ThisSize = %d, destination = %X"),
				FAndroidTLS::GetCurrentThreadId(), File->Handle.Get(),
				*(File->Path), int32(ThisSize), Destination);
#endif
			if (ThisSize < 0)
			{
				return false;
			}
			else if (ThisSize == 0)
			{
				break;
			}
			TotalBytesRead += ThisSize;
			Offset += ThisSize;
			Destination += ThisSize;
			BytesToRead -= ThisSize;
		}

		return BytesToRead == 0;
	}
};


class FAndroidFileManifestReader
{
private:
	bool bInitialized;
	FString ManifestFileName;
	TMap<FString, FDateTime> ManifestEntries;
	FCriticalSection ManifestEntriesCS;
public:

	FAndroidFileManifestReader( const FString& InManifestFileName ) : ManifestFileName(InManifestFileName), bInitialized(false)
	{
	}

	bool GetFileTimeStamp( const FString& FileName, FDateTime& DateTime ) 
	{
		FScopeLock Lock(&ManifestEntriesCS);

		if ( bInitialized == false )
		{
			Read();
			bInitialized = true;
		}

		const FDateTime* Result = ManifestEntries.Find( FileName );
		if ( Result )
		{
			DateTime = *Result;
#if LOG_ANDROID_FILE_MANIFEST
			FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Found time stamp for '%s', %s"), *FileName, *DateTime.ToString());
#endif
			return true;
		}
#if LOG_ANDROID_FILE_MANIFEST
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Didn't find time stamp for '%s'"), *FileName);
#endif
		return false;
	}


	bool SetFileTimeStamp( const FString& FileName, const FDateTime& DateTime )
	{
		FScopeLock Lock(&ManifestEntriesCS);

		if (bInitialized == false)
		{
			Read();
			bInitialized = true;
		}

		FDateTime* Result = ManifestEntries.Find( FileName );
		if ( Result == NULL )
		{
			ManifestEntries.Add(FileName, DateTime );
		}
		else
		{
			*Result = DateTime;
		}
#if LOG_ANDROID_FILE_MANIFEST
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("SetFileTimeStamp '%s', %s"), *FileName, *DateTime.ToString());
#endif
		return true;
	}

	bool DeleteFileTimeStamp(const FString& FileName)
	{
		FScopeLock Lock(&ManifestEntriesCS);

		if (bInitialized == false)
		{
			Read();
			bInitialized = true;
		}

		FDateTime* Result = ManifestEntries.Find(FileName);
		if (Result != NULL)
		{
			ManifestEntries.Remove(FileName);
			
#if LOG_ANDROID_FILE_MANIFEST
			FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Deleted timestamp for file '%s' in manifest file '%s'"), *FileName, *ManifestFileName);
#endif

			return true;
		}
		
		return false;
	}

	// read manifest from disk
	void Read()
	{
		FScopeLock Lock(&ManifestEntriesCS);

		// Local filepaths are directly in the deployment directory.
		static const FString &BasePath = GetFileBasePath();
		const FString ManifestPath = BasePath + ManifestFileName;

		ManifestEntries.Empty();

		FString EntireFile;
		{
			FAndroidUniqueFd Handle { open(TCHAR_TO_UTF8(*ManifestPath), O_RDONLY) };

			if (!Handle.IsValid())
			{
#if LOG_ANDROID_FILE_MANIFEST
				FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Failed to open file for read '%s'"), *ManifestFileName);
#endif
				return;
			}

			int BytesRead = 0;
			{
				ANSICHAR Buffer[1024];
				do
				{
					BytesRead = read(Handle.Get(), Buffer, UE_ARRAY_COUNT(Buffer) - 1);
					if (BytesRead < 0)
					{
						if (errno == EINTR)
						{
							// interrupted by signal, no error
							continue;
						}
						break;
					}
					Buffer[BytesRead] = '\0';
					auto Conv = StringCast<TCHAR>(Buffer);
					EntireFile.Append(Conv.Get(), Conv.Length());
				}
				while (BytesRead > 0);
			}

			if (BytesRead < 0)
			{
#if LOG_ANDROID_FILE_MANIFEST
				const uint32 LastError = FPlatformMisc::GetLastError();
				TCHAR ErrorBuffer[1024];
				FPlatformMisc::GetSystemErrorMessage(ErrorBuffer, UE_ARRAY_COUNT(ErrorBuffer), LastError);
				FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Failed to read the manifest file '%s' (%u: %s)"), *ManifestFileName, LastError, ErrorBuffer);
#endif
				return;
			}
		}

		TArray<FString> Lines;
		EntireFile.ParseIntoArrayLines(Lines);

#if LOG_ANDROID_FILE_MANIFEST
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Loaded manifest file %s"), *ManifestFileName);
		for( const auto& Line : Lines )
		{
			FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Manifest Line %s"), *Line );
		}
#endif
		for ( const auto& Line : Lines) 
		{
#if LOG_ANDROID_FILE_MANIFEST
			FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Processing line '%s' "), *Line);
#endif
			FString Filename;
			FString DateTimeString;
			if ( Line.Split(TEXT("\t"), &Filename, &DateTimeString) )
			{
				FDateTime ModifiedDate;
				if ( FDateTime::ParseIso8601(*DateTimeString, ModifiedDate) )
				{
#if LOG_ANDROID_FILE_MANIFEST
					FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Read time stamp '%s' %s"), *Filename, *ModifiedDate.ToString());
#endif
					Filename.ReplaceInline( TEXT("\\"), TEXT("/") );
					ManifestEntries.Emplace( MoveTemp(Filename), ModifiedDate );
				}
				else
				{
#if LOG_ANDROID_FILE_MANIFEST
					FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Failed to parse date for file '%s' %s"), *Filename, *DateTimeString);
#endif					
				}
			}
#if LOG_ANDROID_FILE_MANIFEST
			else
			{
				FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Unable to split line '%s'"), *Line);
			}
#endif
		}
	}

	void Write()
	{
		FScopeLock Lock(&ManifestEntriesCS);

		// Local filepaths are directly in the deployment directory.
		static const FString &BasePath = GetFileBasePath();
		const FString ManifestPath = BasePath + ManifestFileName;


		FAndroidUniqueFd Handle { open(TCHAR_TO_UTF8(*ManifestPath), O_WRONLY | O_CREAT | O_TRUNC, S_IRUSR | S_IWUSR) };

		if (!Handle.IsValid())
		{
#if LOG_ANDROID_FILE_MANIFEST
			FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Failed to open file for write '%s'"), *ManifestFileName);
#endif
			return;
		}

		for (const auto& Line : ManifestEntries)
		{
			bool bSuccess = true;
			{
				const FString RawDateTimeString = Line.Value.ToIso8601();

				ANSICHAR Buffer[4096] = {0};
				FCStringAnsi::Snprintf(
					Buffer,
					UE_ARRAY_COUNT(Buffer),
					"%s\t%s\r\n",
					StringCast<UTF8CHAR>(*Line.Key).Get(),
					StringCast<UTF8CHAR>(*RawDateTimeString).Get());

				ANSICHAR* Source = Buffer;
				int64 BytesToWrite = FCStringAnsi::Strlen(Source);
				while (BytesToWrite > 0)
				{
					int64 Result = write(Handle.Get(), Source, BytesToWrite);
					if (Result <= 0)
					{
						if (errno == EINTR)
						{
							// interrupted by signal, no error
							continue;
						}
						bSuccess = false;
						break;
					}

					Source       += Result;
					BytesToWrite -= Result;
				}
			}
			if (!bSuccess)
			{
#if LOG_ANDROID_FILE_MANIFEST
				const uint32 LastError = FPlatformMisc::GetLastError();
				TCHAR ErrorBuffer[1024];
				FPlatformMisc::GetSystemErrorMessage(ErrorBuffer, UE_ARRAY_COUNT(ErrorBuffer), LastError);
				FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Failed to write the manifest file '%s' (%u: %s)"), *ManifestFileName, LastError, ErrorBuffer);
#endif
				return;
			}
		}
	}
};

FAndroidFileManifestReader NonUFSManifest(TEXT("Manifest_NonUFSFiles_Android.txt"));
FAndroidFileManifestReader UFSManifest(TEXT("Manifest_UFSFiles_Android.txt"));

/*
	Access to files in multiple ZIP archives.
*/
class FZipUnionFile
{
public:
	struct Entry
	{
		TSharedPtr<FFileHandleAndroid> File;
		FString FileName;
		int32 ModTime = 0;
		bool IsDirectory = false;

		Entry(TSharedPtr<FFileHandleAndroid> file, const FString & filename, int32 modtime, bool isdir)
			: File(file), FileName(filename), ModTime(modtime), IsDirectory(isdir)
		{
		}
	};

	typedef TMap<FString, TSharedPtr<Entry>> FEntryMap;

	struct Directory
	{
		FEntryMap::TIterator Current;
		FString Path;

		Directory(FEntryMap& Entries, const FString& DirPath)
			: Current(Entries.CreateIterator()), Path(DirPath)
		{
			if (!Path.IsEmpty())
			{
				Path /= TEXT("");
			}
			// This would be much easier, and efficient, if TMap
			// supported getting iterators to found entries in
			// the map. Instead we need to iterate the entire
			// map to find the initial directory entry.
			while (Current && Current.Key() != Path)
			{
				++Current;
			}
		}

		bool Next()
		{
			for (++Current; Current; ++Current)
			{
				if (Current.Key().StartsWith(Path))
				{
					int32 i = Current.Key().Find(TEXT("/"), ESearchCase::CaseSensitive, ESearchDir::FromStart, Path.Len());
					if (i == INDEX_NONE || i == Current.Key().Len() - 1)
					{
						break;
					}
				}
			}
			return !!Current;
		}
	};

	FZipUnionFile()
	{
	}

	~FZipUnionFile()
	{
	}

	void AddPatchFile(TSharedPtr<FFileHandleAndroid> file)
	{
		int64 FileLength = file->Size();

		// Is it big enough to be a ZIP?
		check( FileLength >= kEOCDLen );

		int64 ReadAmount = kMaxEOCDSearch;
		if (ReadAmount > FileLength)
		{
			ReadAmount = FileLength;
		}

		// Check magic signature for ZIP.
		file->Seek(0);
		uint32 Header;
		verify( file->Read((uint8*)&Header, sizeof(Header)) );
		check( Header != kEOCDSignature );
		check( Header == kLFHSignature );

		/*
		* Perform the traditional EOCD snipe hunt. We're searching for the End
		* of Central Directory magic number, which appears at the start of the
		* EOCD block. It's followed by 18 bytes of EOCD stuff and up to 64KB of
		* archive comment. We need to read the last part of the file into a
		* buffer, dig through it to find the magic number, parse some values
		* out, and use those to determine the extent of the CD. We start by
		* pulling in the last part of the file.
		*/
		int64 SearchStart = FileLength - ReadAmount;
		ByteBuffer Buffer(ReadAmount);
		verify( file->Seek(SearchStart) );
		verify( file->Read(Buffer.Data, Buffer.Size) );

		/*
		* Scan backward for the EOCD magic. In an archive without a trailing
		* comment, we'll find it on the first try. (We may want to consider
		* doing an initial minimal read; if we don't find it, retry with a
		* second read as above.)
		*/
		int64 EOCDIndex = Buffer.Size - kEOCDLen;
		for (; EOCDIndex >= 0; --EOCDIndex)
		{
			if (Buffer.GetValue<uint32>(EOCDIndex) == kEOCDSignature)
			{
				break;
			}
		}
		check( EOCDIndex >= 0 );

		/*
		* Grab the CD offset and size, and the number of entries in the
		* archive.
		*/
		uint16 NumEntries = (Buffer.GetValue<uint16>(EOCDIndex + kEOCDNumEntries));
		int64 DirSize = (Buffer.GetValue<uint32>(EOCDIndex + kEOCDSize));
		int64 DirOffset = (Buffer.GetValue<uint32>(EOCDIndex + kEOCDFileOffset));
		check( DirOffset + DirSize <= FileLength );
		check( NumEntries > 0 );

		uint16 CommentLength = (Buffer.GetValue<uint16>(EOCDIndex + kEOCDCommentLen));
		if (CommentLength > 0)
		{
			GLastOBBComment = FString(CommentLength, reinterpret_cast<const ANSICHAR*>(Buffer.Data + EOCDIndex + kEOCDCommentStart));
		}
		else
		{
			GLastOBBComment = FString();
		}


		/*
		* Walk through the central directory, adding entries to the hash table.
		*/
		FFileHandleAndroid DirectoryMap(*file, DirOffset, DirSize);
		int64 Offset = 0;
		for (uint16 EntryIndex = 0; EntryIndex < NumEntries; ++EntryIndex)
		{
			uint32 Signature;
			verify( DirectoryMap.Seek(Offset) );
			verify( DirectoryMap.Read((uint8*)&Signature, sizeof(Signature)) );

			// NumEntries may be 65535 so also stop if signature invalid.
			if (Signature != kCDESignature)
			{
				// Hit the end of the central directory, stop.
				break;
			}

			// Entry information. Note, we try and read in incremental
			// order to avoid missing read-aheads.

			uint16 Method;
			verify( DirectoryMap.Seek(Offset + kCDEMethod) );
			verify( DirectoryMap.Read((uint8*)&Method, sizeof(Method)) );

			int32 WhenModified;
			verify( DirectoryMap.Seek(Offset + kCDEModWhen) );
			verify( DirectoryMap.Read((uint8*)&WhenModified, sizeof(WhenModified)) );

			uint32 UncompressedLength;
			verify( DirectoryMap.Seek(Offset + kCDEUncompLen) );
			verify( DirectoryMap.Read((uint8*)&UncompressedLength, sizeof(UncompressedLength)) );

			uint16 FileNameLen;
			verify( DirectoryMap.Seek(Offset + kCDENameLen) );
			verify( DirectoryMap.Read((uint8*)&FileNameLen, sizeof(FileNameLen)) );

			uint16 ExtraLen;
			verify( DirectoryMap.Seek(Offset + kCDEExtraLen) );
			verify( DirectoryMap.Read((uint8*)&ExtraLen, sizeof(ExtraLen)) );

			uint16 CommentLen;
			verify( DirectoryMap.Seek(Offset + kCDECommentLen) );
			verify( DirectoryMap.Read((uint8*)&CommentLen, sizeof(CommentLen)) );

			// We only add uncompressed entries as we don't support decompression.
			if (Method == kCompressStored)
			{
				uint32 LocalOffset;
				verify( DirectoryMap.Seek(Offset + kCDELocalOffset) );
				verify( DirectoryMap.Read((uint8*)&LocalOffset, sizeof(LocalOffset)) );

				ByteBuffer FileNameBuffer(FileNameLen+1);
				verify( DirectoryMap.Seek(Offset + kCDELen) );
				verify( DirectoryMap.Read(FileNameBuffer.Data, FileNameBuffer.Size) );
				FileNameBuffer.Data[FileNameBuffer.Size-1] = 0;
				FString FileName(UTF8_TO_TCHAR(FileNameBuffer.Data));

				verify( file->Seek(LocalOffset) );

				uint32 LocalSignature;
				verify( file->Read((uint8*)&LocalSignature, sizeof(LocalSignature)) );

				uint16 LocalFileNameLen;
				verify( file->Seek(LocalOffset + kLFHNameLen) );
				verify( file->Read((uint8*)&LocalFileNameLen, sizeof(LocalFileNameLen)) );

				uint16 LocalExtraLen;
				verify( file->Seek(LocalOffset + kLFHExtraLen) );
				verify( file->Read((uint8*)&LocalExtraLen, sizeof(LocalExtraLen)) );

				int64 EntryOffset = LocalOffset + kLFHLen + LocalFileNameLen + LocalExtraLen;

				// Add the entry.
#if LOG_ANDROID_FILE
				FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FUnionZipFile::AddPatchFile.. FILE: '%s'"), *FileName);
#endif
				if (FileName.EndsWith(TEXT("/"))) // We only care about actual files in the zip
				{
					check(UncompressedLength == 0);
				}
				else
				{
					Entries.Add(
						FileName,
						MakeShareable(new Entry(
							MakeShareable(new FFileHandleAndroid(
								*file, EntryOffset, UncompressedLength)),
							FileName,
							WhenModified,
							false)));

					// Add extra directory entries to contain the file
					// that we can use to later discover directory contents.
					FileName = FPaths::GetPath(FileName);
					while (!FileName.IsEmpty())
					{
						FString DirName = FileName + "/";
						if (!Entries.Contains(DirName))
						{
#if LOG_ANDROID_FILE
							FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FUnionZipFile::AddPatchFile.. DIR: '%s'"), *DirName);
#endif
							Entries.Add(
								DirName,
								MakeShareable(new Entry(
									nullptr, DirName, 0, true))
							);
						}
						FileName = FPaths::GetPath(FileName);
					}
				}
			}

			// Skip to next entry.
			Offset += kCDELen + FileNameLen + ExtraLen + CommentLen;
		}

		// Keep the entries sorted so that we can do iteration to discover
		// directory contents, and other stuff.
		Entries.KeySort(TLess<FString>());
	}

	bool HasEntry(const FString & Path)
	{
		return Entries.Contains(Path);
	}

	Entry & GetEntry(const FString & Path)
	{
		return *Entries[Path];
	}

	int64 GetEntryLength(const FString & Path)
	{
		TSharedPtr<FFileHandleAndroid>& File = Entries[Path]->File;
		return File != nullptr ? File->Size() : 0;
	}

	int64 GetEntryModTime(const FString & Path)
	{
		return Entries[Path]->ModTime;
	}

	Directory OpenDirectory(const FString & Path)
	{
		return Directory(Entries, Path);
	}

	AAsset * GetEntryAsset(const FString & Path)
	{
		return Entries[Path]->File->File->Asset;
	}

	FString GetEntryRootPath(const FString & Path)
	{
		return Entries[Path]->File->File->Path;
	}

private:
	FEntryMap Entries;

	// Zip file constants.

	const uint32 kEOCDSignature = 0x06054b50;
	const uint32 kEOCDLen = 22;
	const uint32 kEOCDNumEntries = 8; // offset to #of entries in file
	const uint32 kEOCDSize = 12; // size of the central directory
	const uint32 kEOCDFileOffset = 16; // offset to central directory
	const uint32 kEOCDCommentLen = 20; // offset to comment length (ushort)
	const uint32 kEOCDCommentStart = 22; // offset to start of optional comment

	const uint32 kMaxCommentLen = 65535; // longest possible in ushort
	const uint32 kMaxEOCDSearch = (kMaxCommentLen + kEOCDLen);

	const uint32 kLFHSignature = 0x04034b50;
	const uint32 kLFHLen = 30; // excluding variable-len fields
	const uint32 kLFHNameLen = 26; // offset to filename length
	const uint32 kLFHExtraLen = 28; // offset to extra length

	const uint32 kCDESignature = 0x02014b50;
	const uint32 kCDELen = 46; // excluding variable-len fields
	const uint32 kCDEMethod = 10; // offset to compression method
	const uint32 kCDEModWhen = 12; // offset to modification timestamp
	const uint32 kCDECRC = 16; // offset to entry CRC
	const uint32 kCDECompLen = 20; // offset to compressed length
	const uint32 kCDEUncompLen = 24; // offset to uncompressed length
	const uint32 kCDENameLen = 28; // offset to filename length
	const uint32 kCDEExtraLen = 30; // offset to extra length
	const uint32 kCDECommentLen = 32; // offset to comment length
	const uint32 kCDELocalOffset = 42; // offset to local hdr

	const uint32 kCompressStored = 0; // no compression
	const uint32 kCompressDeflated = 8; // standard deflate

	struct ByteBuffer
	{
		uint8* Data;
		int64 Size;

		ByteBuffer(int64 size)
			: Data(new uint8[size]), Size(size)
		{
		}

		~ByteBuffer()
		{
			delete[] Data;
		}

		template <typename T>
		T GetValue(int64 offset)
		{
			return *reinterpret_cast<T*>(this->Data + offset);
		}
	};
};


class FAndroidMappedFileRegion final : public IMappedFileRegion
{
public:
	class FAndroidMappedFileHandle* Parent;
	const uint8* AlignedPtr;
	uint64 AlignedSize;
	FAndroidMappedFileRegion(const uint8* InMappedPtr, const uint8* InAlignedPtr, size_t InMappedSize, uint64 InAlignedSize, const FString& InDebugFilename, size_t InDebugOffsetIntoFile, FAndroidMappedFileHandle* InParent)
		: IMappedFileRegion(InMappedPtr, InMappedSize, InDebugFilename, InDebugOffsetIntoFile)
		, Parent(InParent)
		, AlignedPtr(InAlignedPtr)
		, AlignedSize(InAlignedSize)
	{
	}

	virtual ~FAndroidMappedFileRegion();
};

class FAndroidMappedFileHandle final : public IMappedFileHandle
{
	inline static SIZE_T FileMappingAlignment = FPlatformMemory::GetConstants().PageSize;

public:
	FAndroidMappedFileHandle(FAndroidUniqueFd InFileHandle, int64 FileSize, const FString& InFilename)
		: IMappedFileHandle(FileSize)
		, MappedPtr(nullptr)
		, Filename(InFilename)
		, NumOutstandingRegions(0)
		, FileHandle(MoveTemp(InFileHandle))
	{
	}

	virtual ~FAndroidMappedFileHandle() override
	{
		// can't delete the file before you delete all outstanding regions
		UE_CLOGF(
				NumOutstandingRegions.load(std::memory_order_relaxed),
				LogAndroidFile,
#if UE_BUILD_SHIPPING
				Error,
#else
				Fatal,
#endif
				"Cleaning mapped file with alive mapped regions: %ls",
				*Filename);
	}

	virtual IMappedFileRegion* MapRegion(int64 Offset = 0, int64 BytesToMap = MAX_int64, FFileMappingFlags Flags = EMappedFileFlags::ENone) override
	{
		const int64 CurrentFileSize = GetCurrentFileSize();
		check(Offset < CurrentFileSize); // don't map zero bytes and don't map off the end of the file
		BytesToMap = FMath::Min<int64>(BytesToMap, CurrentFileSize - Offset);
		check(BytesToMap > 0); // don't map zero bytes

		const int64 AlignedOffset = AlignDown(Offset, FileMappingAlignment);
		//File mapping can extend beyond file size. It's OK, kernel will just fill any leftover page data with zeros
		const int64 AlignedSize = Align(BytesToMap + Offset - AlignedOffset, FileMappingAlignment);

		int Protection = PROT_READ;
		int InternalFlags = (EnumHasAnyFlags(Flags.Flags, EMappedFileFlags::EPreloadHint)) ? MAP_POPULATE : 0;
		if (EnumHasAnyFlags(Flags.Flags, EMappedFileFlags::EFileWritable))
		{
			Protection |= PROT_WRITE;
			InternalFlags |= MAP_SHARED;
		}
		else
		{
			InternalFlags |= MAP_PRIVATE;
		}

		const uint8* AlignedMapPtr = static_cast<const uint8*>(mmap(nullptr, AlignedSize, Protection, InternalFlags, FileHandle.Get(), AlignedOffset));
		if (AlignedMapPtr == MAP_FAILED || AlignedMapPtr == nullptr)
		{
#if LOG_ANDROID_FILE
			FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Failed to mmap region from %s, errno=%s"), *Filename, UTF8_TO_TCHAR(strerror(errno)));
#endif
			UE_LOGF(LogAndroidFile, Warning, "Failed to map memory %ls, error is %d", *Filename, errno);
			return nullptr;
		}

		// create a mapping for this range
		const uint8* MapPtr = AlignedMapPtr + Offset - AlignedOffset;
		FAndroidMappedFileRegion* Result = new FAndroidMappedFileRegion(MapPtr, AlignedMapPtr, BytesToMap, AlignedSize, Filename, Offset, this);
		NumOutstandingRegions.fetch_add(1, std::memory_order_relaxed);
		return Result;
	}

	void UnMap(const FAndroidMappedFileRegion* Region)
	{
		int32 OldNumOutstandingRegions = NumOutstandingRegions.fetch_sub(1, std::memory_order_relaxed);
		check(OldNumOutstandingRegions > 0);

		const int Res = munmap(const_cast<uint8*>(Region->AlignedPtr), Region->AlignedSize);
#if LOG_ANDROID_FILE
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Failed to unmap region from %s, errno=%s"), *Filename, UTF8_TO_TCHAR(strerror(errno)));
#endif
		const int64 CurrentFileSize = GetCurrentFileSize();
		checkf(Res == 0, TEXT("Failed to unmap, error is %d, errno is %d [params: %llx, %lld]"), Res, errno, MappedPtr, CurrentFileSize);
	}

private:
	const uint8* MappedPtr;
	FString Filename;
	std::atomic<int32> NumOutstandingRegions;
	FAndroidUniqueFd FileHandle;

	int64 GetCurrentFileSize() const
	{
		struct stat FileInfo;
		FileInfo.st_size = -1;
		const int StatResult = fstat(FileHandle.Get(), &FileInfo);
		if (StatResult == -1)
		{
			const int ErrNo = errno;
#if LOG_ANDROID_FILE
			FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FAndroidPlatformFile::FAndroidMappedFileHandle fstat failed: ('%s') failed: errno=%d (%s)"), *Filename, ErrNo, UTF8_TO_TCHAR(strerror(ErrNo)));
#endif
			return GetFileSize();
		}
		return FileInfo.st_size;
	}
};

FAndroidMappedFileRegion::~FAndroidMappedFileRegion()
{
	Parent->UnMap(this);
}


// NOTE: Files are stored either loosely in the deployment directory
// or packed in an OBB archive. We don't know which one unless we try
// and get the files. We always first check if the files are local,
// i.e. loosely stored in deployment dir, if they aren't we assume
// they are archived (and can use the asset system to get them).

/**
	Implementation for Android file I/O. These handles access to these
	kinds of files:

	1. Plain-old-files in the file system (i.e. sdcard).
	2. Resources packed in OBBs (aka ZIPs) placed in download locations.
	3. Resources packed in OBBs embedded in the APK.
	4. Direct assets packaged in the APK.

	The base filenames are checked in the above order to allow for
	overriding content from the most "frozen" to the most "fluid"
	state. Hence creating a virtual single union file-system.
**/
class CORE_API FAndroidPlatformFile : public IAndroidPlatformFile
{
	// Note: TManagedStoragePlatformFile used below templates a subclass of FAndroidPlatformFile, so it cannot be made `final`.
public:
	// Singleton implementation.
	static FAndroidPlatformFile & GetPlatformPhysical()
	{
#if PLATFORM_USE_PLATFORM_FILE_MANAGED_STORAGE_WRAPPER
		static TManagedStoragePlatformFile<FAndroidPlatformFile> AndroidPlatformSingleton;
#else
		static FAndroidPlatformFile AndroidPlatformSingleton;
#endif
		return AndroidPlatformSingleton;
	}

	FAndroidPlatformFile()
		: AssetMgr(nullptr)
	{
#if USE_ANDROID_JNI
		AssetMgr = AndroidThunkCpp_GetAssetManager();
#endif
	}

	static const FString* GetOverrideLogDirectory()
	{
		return GOverrideAndroidLogDir ? &AndroidLogDir : nullptr;
	}

	// On initialization we search for OBBs that we need to
	// open to find resources.
	virtual bool Initialize(IPlatformFile* Inner, const TCHAR* CmdLine) override
	{
#if LOG_ANDROID_FILE
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FAndroidPlatformFile::Initialize(..)"));
#endif
		if (!IPhysicalPlatformFile::Initialize(Inner, CmdLine))
		{
			FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FAndroidPlatformFile::Initialize failed"));
			return false;
		}
		if (GOBBinAPK)
		{
			// Open the APK as a ZIP
			FZipUnionFile APKZip;
			FAndroidUniqueFd Handle { open(TCHAR_TO_UTF8(*GAPKFilename), O_RDONLY) };

			if (!Handle.IsValid())
			{
				FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FAndroidPlatformFile::Initialize unable to open APK: %s"), *GAPKFilename);
				return false;
			}
			FFileHandleAndroid* APKFile = new FFileHandleAndroid(GAPKFilename, MoveTemp(Handle));
			APKZip.AddPatchFile(MakeShareable(APKFile));

			// Now open the OBB in the APK and mount it
			if (APKZip.HasEntry("assets/main.obb.png"))
			{
				auto OBBEntry = APKZip.GetEntry("assets/main.obb.png");
				FFileHandleAndroid* OBBFile = static_cast<FFileHandleAndroid*>(new FFileHandleAndroid(*OBBEntry.File, 0, OBBEntry.File->Size()));
				check(nullptr != OBBFile);
				ZipResource.AddPatchFile(MakeShareable(OBBFile));
				FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Mounted main OBB in APK: %s"), *GAPKFilename);

				// check for optional patch obb in APK
				if (APKZip.HasEntry("assets/patch.obb.png"))
				{
					auto patchOBBEntry = APKZip.GetEntry("assets/patch.obb.png");
					FFileHandleAndroid* patchOBBFile = static_cast<FFileHandleAndroid*>(new FFileHandleAndroid(*patchOBBEntry.File, 0, patchOBBEntry.File->Size()));
					check(nullptr != patchOBBFile);
					ZipResource.AddPatchFile(MakeShareable(patchOBBFile));
					FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Mounted patch OBB in APK: %s"), *GAPKFilename);
				}
			}
			else
			{
				FPlatformMisc::LowLevelOutputDebugStringf(TEXT("OBB not found in APK: %s"), *GAPKFilename);
				return false;
			}
		}
		else
		{
			// For external OBBs we mount the specific OBB files,
			// main and patch, only. As required by Android specs.
			// See <http://developer.android.com/google/play/expansion-files.html>
			// but first checks for overrides of expected OBB file paths if provided
			FString OBBDir1 = GOBBFilePathBase + FString(TEXT("/Android/obb/") + GPackageName);
			FString OBBDir2 = GOBBFilePathBase + FString(TEXT("/obb/") + GPackageName);
			FString MainOBBName = FString::Printf(TEXT("main.%d.%s.obb"), GAndroidPackageVersion, *GPackageName);
			FString PatchOBBName = FString::Printf(TEXT("patch.%d.%s.obb"), GAndroidPackageVersion, *GPackageName);

			if (!GOBBMainFilePath.IsEmpty() && FileExists(*GOBBMainFilePath, true))
			{
				MountOBB(*GOBBMainFilePath);
				FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Mounted main OBB: %s"), *GOBBMainFilePath);
			}
			else if (FileExists(*(OBBDir1 / MainOBBName), true))
			{
				GOBBMainFilePath = OBBDir1 / MainOBBName;
				MountOBB(*GOBBMainFilePath);
				FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Mounted main OBB: %s"), *GOBBMainFilePath);
			}
			else if (FileExists(*(OBBDir2 / MainOBBName), true))
			{
				GOBBMainFilePath = OBBDir2 / MainOBBName;
				MountOBB(*GOBBMainFilePath);
				FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Mounted main OBB: %s"), *GOBBMainFilePath);
			}

			bool bHavePatch = false;
			if (!GOBBPatchFilePath.IsEmpty() && FileExists(*GOBBPatchFilePath, true))
			{
				bHavePatch = true;
				MountOBB(*GOBBPatchFilePath);
				FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Mounted patch OBB: %s"), *GOBBPatchFilePath);
			}
			else if (FileExists(*(OBBDir1 / PatchOBBName), true))
			{
				bHavePatch = true;
				GOBBPatchFilePath = OBBDir1 / PatchOBBName;
				MountOBB(*GOBBPatchFilePath);
				FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Mounted patch OBB: %s"), *GOBBPatchFilePath);
			}
			else if (FileExists(*(OBBDir2 / PatchOBBName), true))
			{
				bHavePatch = true;
				GOBBPatchFilePath = OBBDir2 / PatchOBBName;
				MountOBB(*GOBBPatchFilePath);
				FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Mounted patch OBB: %s"), *GOBBPatchFilePath);
			}

			// Only check for overflow files if we found a patch file
			if (bHavePatch)
			{
				int32 OverflowIndex = 1;

				if (!GOBBOverflow1FilePath.IsEmpty() && FileExists(*GOBBOverflow1FilePath, true))
				{
					OverflowIndex = 2;
					MountOBB(*GOBBOverflow1FilePath);
					FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Mounted overflow1 OBB: %s"), *GOBBOverflow1FilePath);
				}
				if (!GOBBOverflow2FilePath.IsEmpty() && FileExists(*GOBBOverflow2FilePath, true))
				{
					OverflowIndex = 3;
					MountOBB(*GOBBOverflow2FilePath);
					FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Mounted overflow2 OBB: %s"), *GOBBOverflow2FilePath);
				}

				while (OverflowIndex <= ANDROID_MAX_OVERFLOW_FILES)
				{
					FString OverflowOBBName = FString::Printf(TEXT("overflow%d.%d.%s.obb"), OverflowIndex, GAndroidPackageVersion, *GPackageName);

					if (FileExists(*(OBBDir1 / OverflowOBBName), true))
					{
						FString OBBOverflowFilePath = OBBDir1 / OverflowOBBName;
						MountOBB(*OBBOverflowFilePath);
						FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Mounted overflow%d OBB: %s"), OverflowIndex, *OBBOverflowFilePath);
					}
					else if (FileExists(*(OBBDir2 / OverflowOBBName), true))
					{
						FString OBBOverflowFilePath = OBBDir2 / OverflowOBBName;
						MountOBB(*OBBOverflowFilePath);
						FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Mounted overflow%d OBB: %s"), OverflowIndex, *OBBOverflowFilePath);
					}
					else
					{
						break;
					}

					OverflowIndex++;
				}
			}
		}


		// make sure the base path directory exists (UnrealGame and UnrealGame/ProjectName)
		FString FileBaseDir = GFilePathBase + FString(FILEBASE_DIRECTORY);
		mkdir(TCHAR_TO_UTF8(*FileBaseDir), 0777);
		mkdir(TCHAR_TO_UTF8(*(FileBaseDir + GAndroidProjectName)), 0777);

		// make sure the log directory exists if override applied
		//if (GOverrideAndroidLogDir)
		{
			FString LogBaseDir = GExternalFilePath + FString(FILEBASE_DIRECTORY);
			mkdir(TCHAR_TO_UTF8(*LogBaseDir), 0777);
			mkdir(TCHAR_TO_UTF8(*(LogBaseDir + GAndroidProjectName)), 0777);
			mkdir(TCHAR_TO_UTF8(*(LogBaseDir + GAndroidProjectName + TEXT("/") + GAndroidProjectName)), 0777);
			mkdir(TCHAR_TO_UTF8(*(LogBaseDir + GAndroidProjectName + TEXT("/") + GAndroidProjectName + TEXT("/Saved"))), 0777);
			mkdir(TCHAR_TO_UTF8(*(LogBaseDir + GAndroidProjectName + TEXT("/") + GAndroidProjectName + TEXT("/Saved/Logs"))), 0777);

			AndroidLogDir = LogBaseDir + GAndroidProjectName + TEXT("/") + GAndroidProjectName + TEXT("/Saved/Logs/");
		}

		return true;
	}

	virtual bool FileExists(const TCHAR* Filename) override
	{
		const bool bAllowAssets = false;
		return FileExists(Filename, bAllowAssets);
	}

	virtual bool FileExists(const TCHAR* Filename, bool bAllowAssets) override
	{
		const bool bForceAllowLocal = false;
		return FileExists(Filename, bForceAllowLocal, bAllowAssets);
	}

	bool FileExists(const TCHAR* Filename, bool bForceAllowLocal, bool bAllowAssets)
	{
#if LOG_ANDROID_FILE
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FAndroidPlatformFile::FileExists('%s')"), Filename);
#endif
		FString LocalPath;
		FString AssetPath;
		PathToAndroidPaths(LocalPath, AssetPath, Filename, bForceAllowLocal);

		bool result = false;
		struct stat FileInfo;
		if (!LocalPath.IsEmpty() && (stat(TCHAR_TO_UTF8(*LocalPath), &FileInfo) == 0))
		{
			// For local files we need to check if it's a plain
			// file, as opposed to directories.
			result = S_ISREG(FileInfo.st_mode);
		}
		else
		{
			// For everything else we only check existence.
			result = IsResource(AssetPath) || bAllowAssets && IsAsset(AssetPath);
		}
#if LOG_ANDROID_FILE
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FAndroidPlatformFile::FileExists('%s') => %s\nResolved as %s"), Filename, result ? TEXT("TRUE") : TEXT("FALSE"), *LocalPath);
#endif
		return result;
	}

	virtual FOpenMappedResult OpenMappedEx(const TCHAR* Filename, EOpenReadFlags OpenOptions = EOpenReadFlags::None, int64 MaximumSize = 0) override
	{
		EOpenMappedFlags ActualOpenOptions = EOpenMappedFlags::None;
		if (EnumHasAnyFlags(OpenOptions, EOpenReadFlags::AllowWrite))
		{
			ActualOpenOptions |= EOpenMappedFlags::OpenForWrite;
			ActualOpenOptions |= EOpenMappedFlags::OpenExisting;
		}

		return OpenMappedEx2(Filename, ActualOpenOptions, MaximumSize);
	}

	virtual FOpenMappedResult OpenMappedEx2(const TCHAR* Filename, EOpenMappedFlags OpenOptions = EOpenMappedFlags::None, int64 MaximumSize = 0) override
	{
		FString LocalPath;
		FString AssetPath;
		const bool bForceAllowLocal = false;
		PathToAndroidPaths(LocalPath, AssetPath, Filename, bForceAllowLocal);

		const FString NormalizedFilename = MoveTemp(LocalPath);

		int Flags = O_RDONLY;
		if (EnumHasAnyFlags(OpenOptions, EOpenMappedFlags::OpenForWrite))
		{
			Flags = O_RDWR;
			if (!EnumHasAnyFlags(OpenOptions, EOpenMappedFlags::OpenExisting))
			{
				Flags |= O_CREAT;
				Flags |= O_TRUNC;
			}
		}

		int32 iHandle = -1;
		if (EnumHasAnyFlags(OpenOptions, EOpenMappedFlags::OpenForWrite))
		{
			iHandle = open(TCHAR_TO_UTF8(*NormalizedFilename), Flags, S_IRUSR | S_IWUSR | S_IRGRP | S_IWGRP);
		}
		else
		{
			iHandle = open(TCHAR_TO_UTF8(*NormalizedFilename), Flags);
		}

		FAndroidUniqueFd Handle{ iHandle };
		if (!Handle.IsValid())
		{
			const int ErrNo = errno;
			FString ErrorStr = FString::Printf(TEXT("FAndroidPlatformFile::OpenMappedEx('%s', Flags=0x%08X) failed: errno=%d (%s)"), *NormalizedFilename, Flags, ErrNo, UTF8_TO_TCHAR(strerror(ErrNo)));
#if LOG_ANDROID_FILE
			FPlatformMisc::LowLevelOutputDebugString(*ErrorStr);
#endif
			return MakeError(MoveTemp(ErrorStr));
		}

		struct stat FileInfo;
		FileInfo.st_size = -1;
		const int StatResult = fstat(Handle.Get(), &FileInfo);
		if (StatResult == -1)
		{
			const int ErrNo = errno;
			FString ErrorStr = FString::Printf(TEXT("FAndroidPlatformFile::OpenMappedEx fstat failed: ('%s', Flags=0x%08X) failed: errno=%d (%s)"), *NormalizedFilename, Flags, ErrNo, UTF8_TO_TCHAR(strerror(ErrNo)));

#if LOG_ANDROID_FILE
			FPlatformMisc::LowLevelOutputDebugString(*ErrorStr);
#endif

			return MakeError(MoveTemp(ErrorStr));
		}

		if (EnumHasAnyFlags(OpenOptions, EOpenMappedFlags::OpenForWrite))
		{
			if (MaximumSize < 1)
			{
				if (EnumHasAnyFlags(OpenOptions, EOpenMappedFlags::OpenExisting))
				{
					if (FileInfo.st_size < 1)
					{
						return MakeError(TEXT("File size is 0"));
					}

					MaximumSize = FileInfo.st_size;
				}
				else
				{
					return MakeError(TEXT("No maximum size provided."));
				}
			}
			else if (MaximumSize > FileInfo.st_size)
			{
				int TruncResult = 0;
				do { TruncResult = __ftruncate(Handle.Get(), MaximumSize); } while (TruncResult < 0 && errno == EINTR);

				if (TruncResult == -1)
				{
					const int ErrNo = errno;
					FString ErrorStr = FString::Printf(TEXT("FAndroidPlatformFile::OpenMappedEx ftruncate failed: ('%s', Flags=0x%08X) failed: errno=%d (%s)"), *NormalizedFilename, Flags, ErrNo, UTF8_TO_TCHAR(strerror(ErrNo)));

#if LOG_ANDROID_FILE
					FPlatformMisc::LowLevelOutputDebugString(*ErrorStr);
#endif

					return MakeError(MoveTemp(ErrorStr));
				}
			}
		}
		else
		{
			if (FileInfo.st_size < 1)
			{
				return MakeError(TEXT("File size is 0"));
			}

			if (MaximumSize < 1 || MaximumSize > FileInfo.st_size)
			{
				MaximumSize = FileInfo.st_size;
			}
		}

		return MakeValue(new FAndroidMappedFileHandle(MoveTemp(Handle), MaximumSize, NormalizedFilename));
	}

	virtual int64 FileSize(const TCHAR* Filename) override
	{
		const bool bAllowAssets = false;
		return FileSize(Filename, bAllowAssets);
	}

	virtual int64 FileSize(const TCHAR* Filename, bool bAllowAssets) override
	{
		const bool bForceAllowLocal = false;
		return FileSize(Filename, bForceAllowLocal, bAllowAssets);
	}

	int64 FileSize(const TCHAR* Filename, bool bForceAllowLocal, bool bAllowAssets)
	{
#if LOG_ANDROID_FILE
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FAndroidPlatformFile::FileSize('%s')"), Filename);
#endif
		FString LocalPath;
		FString AssetPath;
		PathToAndroidPaths(LocalPath, AssetPath, Filename, bForceAllowLocal);

		struct stat FileInfo;
		FileInfo.st_size = -1;
		if (!LocalPath.IsEmpty() && (stat(TCHAR_TO_UTF8(*LocalPath), &FileInfo) == 0))
		{
			// make sure to return -1 for directories
			if (S_ISDIR(FileInfo.st_mode))
			{
				FileInfo.st_size = -1;
			}
			return FileInfo.st_size;
		}
		else if (IsResource(AssetPath))
		{
			FileInfo.st_size = ZipResource.GetEntryLength(AssetPath);
		}
		else if (bAllowAssets)
		{
			AAsset * file = AAssetManager_open(AssetMgr, TCHAR_TO_UTF8(*AssetPath), AASSET_MODE_UNKNOWN);
			if (nullptr != file)
			{
				FileInfo.st_size = AAsset_getLength(file);
				AAsset_close(file);
			}
		}
		return FileInfo.st_size;
	}

	virtual bool DeleteFile(const TCHAR* Filename) override
	{
		const bool bForceAllowLocal = false;
		return DeleteFile(Filename, bForceAllowLocal);
	}

	bool DeleteFile(const TCHAR* Filename, bool bForceAllowLocal)
	{
#if LOG_ANDROID_FILE
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FAndroidPlatformFile::DeleteFile('%s')"), Filename);
#endif
		FString LocalPath;
		FString AssetPath;
		PathToAndroidPaths(LocalPath, AssetPath, Filename, bForceAllowLocal);

		// Only delete if we have a local file.
		if (IsLocal(LocalPath))
		{
#if !USE_UTIME
			if (NonUFSManifest.DeleteFileTimeStamp(AssetPath))
			{
				NonUFSManifest.Write();
			}
			else if (UFSManifest.DeleteFileTimeStamp(AssetPath))
			{
				UFSManifest.Write();
			}
#endif
			return unlink(TCHAR_TO_UTF8(*LocalPath)) == 0;
		}
		return false;
	}

	// NOTE: Returns false if the file is not found.
	virtual bool IsReadOnly(const TCHAR* Filename) override
	{
		const bool bForceAllowLocal = false;
		return IsReadOnly(Filename, bForceAllowLocal);
	}

	bool IsReadOnly(const TCHAR* Filename, bool bForceAllowLocal)
	{
#if LOG_ANDROID_FILE
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FAndroidPlatformFile::IsReadOnly('%s')"), Filename);
#endif
		FString LocalPath;
		FString AssetPath;
		PathToAndroidPaths(LocalPath, AssetPath, Filename, bForceAllowLocal);

		if (IsLocal(LocalPath))
		{
			if (access(TCHAR_TO_UTF8(*LocalPath), W_OK) == -1)
			{
				return errno == EACCES;
			}
		}
		else
		{
			// Anything other than local files are from read-only sources.
			return IsResource(AssetPath) || IsAsset(AssetPath);
		}
		return false;
	}

	virtual bool MoveFile(const TCHAR* To, const TCHAR* From) override
	{
		const bool bForceAllowLocal = false;
		return MoveFile(To, From, bForceAllowLocal);
	}

	bool MoveFile(const TCHAR* To, const TCHAR* From, bool bForceAllowLocal)
	{
#if LOG_ANDROID_FILE
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FAndroidPlatformFile::MoveFile('%s', '%s')"), To, From);
#endif
		// Can only move local files.
		FString ToLocalPath;
		FString ToAssetPath;
		PathToAndroidPaths(ToLocalPath, ToAssetPath, To, bForceAllowLocal);
		FString FromLocalPath;
		FString FromAssetPath;
		PathToAndroidPaths(FromLocalPath, FromAssetPath, From, bForceAllowLocal);

		if (IsLocal(FromLocalPath))
		{
			bool bSuccess = rename(TCHAR_TO_UTF8(*FromLocalPath), TCHAR_TO_UTF8(*ToLocalPath)) != -1;
			if (!bSuccess)
			{
#if LOG_ANDROID_FILE
				FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FAndroidPlatformFile::MoveFile() - failed = %s"), UTF8_TO_TCHAR(strerror(errno)));
#endif
			}
			return bSuccess;
		}
		return false;
	}

	virtual bool SetReadOnly(const TCHAR* Filename, bool bNewReadOnlyValue) override
	{
		const bool bForceAllowLocal = false;
		return SetReadOnly(Filename, bNewReadOnlyValue, bForceAllowLocal);
	}

	bool SetReadOnly(const TCHAR* Filename, bool bNewReadOnlyValue, bool bForceAllowLocal)
	{
#if LOG_ANDROID_FILE
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FAndroidPlatformFile::SetReadOnly('%s')"), Filename);
#endif
		FString LocalPath;
		FString AssetPath;
		PathToAndroidPaths(LocalPath, AssetPath, Filename, bForceAllowLocal);

		if (IsLocal(LocalPath))
		{
			struct stat FileInfo;
			if (stat(TCHAR_TO_UTF8(*LocalPath), &FileInfo) != -1)
			{
				if (bNewReadOnlyValue)
				{
					FileInfo.st_mode &= ~S_IWUSR;
				}
				else
				{
					FileInfo.st_mode |= S_IWUSR;
				}
				return chmod(TCHAR_TO_UTF8(*LocalPath), FileInfo.st_mode) == 0;
			}
		}
		return false;
	}

	virtual FDateTime GetTimeStamp(const TCHAR* Filename) override
	{
		const bool bForceAllowLocal = false;
		return GetTimeStamp(Filename, bForceAllowLocal);
	}

	FDateTime GetTimeStamp(const TCHAR* Filename, bool bForceAllowLocal)
	{
#if LOG_ANDROID_FILE_MANIFEST
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FAndroidPlatformFile::GetTimeStamp('%s')"), Filename);
#endif
		FString LocalPath;
		FString AssetPath;
		PathToAndroidPaths(LocalPath, AssetPath, Filename, bForceAllowLocal);

		if (IsLocal(LocalPath))
		{

#if USE_UTIME
			struct stat FileInfo;
			if (stat(TCHAR_TO_UTF8(*LocalPath), &FileInfo) == -1)
			{
				return FDateTime::MinValue();
			}
			// convert _stat time to FDateTime
			FTimespan TimeSinceEpoch(0, 0, FileInfo.st_mtime);
			return AndroidEpoch + TimeSinceEpoch;
#else
			FDateTime Result;
			if ( NonUFSManifest.GetFileTimeStamp(AssetPath,Result) )
			{
				return Result;
			}

			if ( UFSManifest.GetFileTimeStamp( AssetPath, Result ) )
			{
				return Result;
			}

#if LOG_ANDROID_FILE_MANIFEST
			FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Failed to find time stamp in NonUFSManifest for file '%s'"), Filename);
#endif

			// pak file outside of obb may not be in manifest so check if it exists
			if (AssetPath.EndsWith(".pak"))
			{
				// return local file access timestamp (if exists)
				const bool bPakForceAllowLocal = true;
				return GetAccessTimeStamp(Filename, bPakForceAllowLocal);
			}

			return FDateTime::MinValue();

#endif
		}
		else if (IsResource(AssetPath))
		{
			FTimespan TimeSinceEpoch(0, 0, ZipResource.GetEntryModTime(AssetPath));
			return AndroidEpoch + TimeSinceEpoch;
		}
		else
		{
			// No TimeStamp for assets, so just return a default timespan for now.
			return FDateTime::MinValue();
		}
	}

	virtual void SetTimeStamp(const TCHAR* Filename, const FDateTime DateTime) override
	{
		const bool bForceAllowLocal = false;
		return SetTimeStamp(Filename, DateTime, bForceAllowLocal);
	}
	
	void SetTimeStamp(const TCHAR* Filename, const FDateTime DateTime, bool bForceAllowLocal)
	{
#if LOG_ANDROID_FILE
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FAndroidPlatformFile::SetTimeStamp('%s')"), Filename);
#endif
		FString LocalPath;
		FString AssetPath;
		PathToAndroidPaths(LocalPath, AssetPath, Filename, bForceAllowLocal);

		// Can only set time stamp on local files
		if (IsLocal(LocalPath))
		{
#if USE_UTIME
			// get file times
			struct stat FileInfo;
			if (stat(TCHAR_TO_UTF8(*LocalPath), &FileInfo) == -1)
			{
				return;
			}
			// change the modification time only
			struct utimbuf Times;
			Times.actime = FileInfo.st_atime;
			Times.modtime = (DateTime - AndroidEpoch).GetTotalSeconds();
			utime(TCHAR_TO_UTF8(*LocalPath), &Times);
#else
			// do something better as utime isn't supported on android very well...
			FDateTime TempDateTime;
			if ( NonUFSManifest.GetFileTimeStamp( AssetPath, TempDateTime ) )
			{
				NonUFSManifest.SetFileTimeStamp( AssetPath, DateTime );
				NonUFSManifest.Write();
			}
			else
			{
				UFSManifest.SetFileTimeStamp( AssetPath, DateTime );
				UFSManifest.Write();
			}
#endif
		}
	}

	virtual FDateTime GetAccessTimeStamp(const TCHAR* Filename) override
	{
		const bool bForceAllowLocal = false;
		return GetAccessTimeStamp(Filename, bForceAllowLocal);
	}

	FDateTime GetAccessTimeStamp(const TCHAR* Filename, bool bForceAllowLocal)
	{
#if LOG_ANDROID_FILE
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FAndroidPlatformFile::GetAccessTimeStamp('%s')"), Filename);
#endif
		FString LocalPath;
		FString AssetPath;
		PathToAndroidPaths(LocalPath, AssetPath, Filename, bForceAllowLocal);

		if (IsLocal(LocalPath))
		{
			struct stat FileInfo;
			if (stat(TCHAR_TO_UTF8(*LocalPath), &FileInfo) == -1)
			{
				return FDateTime::MinValue();
			}
			// convert _stat time to FDateTime
			FTimespan TimeSinceEpoch(0, 0, FileInfo.st_atime);
			return AndroidEpoch + TimeSinceEpoch;
		}
		else
		{
			// No TimeStamp for resources nor assets, so just return a default timespan for now.
			return FDateTime::MinValue();
		}
	}

	virtual FFileStatData GetStatData(const TCHAR* FilenameOrDirectory) override
	{
		const bool bAllowAssets = false;
		return GetStatData(FilenameOrDirectory, bAllowAssets);
	}

	virtual FFileStatData GetStatData(const TCHAR* FilenameOrDirectory, bool bAllowAssets) override
	{
		const bool bForceAllowLocal = false;
		return GetStatData(FilenameOrDirectory, bForceAllowLocal, bAllowAssets);
	}

	FFileStatData GetStatData(const TCHAR* FilenameOrDirectory, bool bForceAllowLocal, bool bAllowAssets)
	{
#if LOG_ANDROID_FILE
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FAndroidPlatformFile::GetStatData('%s')"), FilenameOrDirectory);
#endif
		FString LocalPath;
		FString AssetPath;
		PathToAndroidPaths(LocalPath, AssetPath, FilenameOrDirectory, bForceAllowLocal);

		if (IsLocal(LocalPath))
		{
			struct stat FileInfo;
			if (stat(TCHAR_TO_UTF8(*LocalPath), &FileInfo) != -1)
			{
				return AndroidStatToUEFileData(FileInfo);
			}
		}
		else if (IsResource(AssetPath))
		{
			return FFileStatData(
				FDateTime::MinValue(),					// CreationTime
				FDateTime::MinValue(),					// AccessTime
				FDateTime::MinValue(),					// ModificationTime
				ZipResource.GetEntryLength(AssetPath),	// FileSize
				false,									// bIsDirectory
				true									// bIsReadOnly
				);
		}
		else if (bAllowAssets)
		{
			AAsset * file = AAssetManager_open(AssetMgr, TCHAR_TO_UTF8(*AssetPath), AASSET_MODE_UNKNOWN);
			bool isDirectory = false;
			bool exists = (nullptr != file);
			int64 FileSize = -1;
			if (exists)
			{
				FileSize = AAsset_getLength(file);
				AAsset_close(file);
			}
			else
			{
				AAssetDir* dir = OpenExistingAssetManagerDirectory(AssetMgr, TCHAR_TO_UTF8(*AssetPath));
				exists = (nullptr != dir);
				
				if (exists)
				{
					isDirectory = true;
					AAssetDir_close(dir);
				}
			}
			
			if (exists)
			{
				return FFileStatData(
					FDateTime::MinValue(),				// CreationTime
					FDateTime::MinValue(),				// AccessTime
					FDateTime::MinValue(),				// ModificationTime
					FileSize,							// FileSize
					isDirectory,						// bIsDirectory
					true								// bIsReadOnly
					);
			}
		}
		
		return FFileStatData();
	}

	virtual FString GetFilenameOnDisk(const TCHAR* Filename) override
	{
		return Filename;
	}

	virtual IFileHandle* OpenRead(const TCHAR* Filename, bool bAllowWrite = false) override
	{
		const bool bForceAllowLocal = false;
		return OpenReadInternal(Filename, bForceAllowLocal, bAllowWrite);
	}

	IFileHandle* OpenReadInternal(const TCHAR* Filename, bool bForceAllowLocal, bool bAllowWrite)
	{
#if LOG_ANDROID_FILE
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FAndroidPlatformFile::OpenRead('%s')"), Filename);
#endif
		FString LocalPath;
		FString AssetPath;
		PathToAndroidPaths(LocalPath, AssetPath, Filename, bForceAllowLocal);

		if (IsLocal(LocalPath))
		{
			FAndroidUniqueFd Handle { open(TCHAR_TO_UTF8(*LocalPath), O_RDONLY) };
			if (Handle.IsValid())
			{
				return new FFileHandleAndroid(LocalPath, MoveTemp(Handle));
			}
		}
		else if (IsResource(AssetPath))
		{
			return new FFileHandleAndroid(
				*ZipResource.GetEntry(AssetPath).File,
				0, ZipResource.GetEntry(AssetPath).File->Size());
		}
		else
		{
			AAsset * asset = AAssetManager_open(AssetMgr, TCHAR_TO_UTF8(*AssetPath), AASSET_MODE_RANDOM);
			if (nullptr != asset)
			{
				return new FFileHandleAndroid(AssetPath, asset);
			}
		}
		return nullptr;
	}

	// Regardless of the file being local, asset, or resource, we
	// assert that opening a file for write will open a local file.
	// The intent is to allow creating fresh files that override
	// packaged content.
	virtual IFileHandle* OpenWrite(const TCHAR* Filename, bool bAppend, bool bAllowRead) override
	{
		const bool bForceAllowLocal = false;
		return OpenWriteInternal(Filename, bAppend, bAllowRead, bForceAllowLocal);
	}

	IFileHandle* OpenWriteInternal(const TCHAR* Filename, bool bAppend, bool bAllowRead, bool bForceAllowLocal)
	{
#if LOG_ANDROID_FILE
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FAndroidPlatformFile::OpenWrite('%s')"), Filename);
#endif
		FString LocalPath;
		FString AssetPath;
		PathToAndroidPaths(LocalPath, AssetPath, Filename, bForceAllowLocal);

		int Flags = O_CREAT;
		if (!bAppend)
		{
			Flags |= O_TRUNC;
		}
		if (bAllowRead)
		{
			Flags |= O_RDWR;
		}
		else
		{
			Flags |= O_WRONLY;
		}

		FAndroidUniqueFd Handle { open(TCHAR_TO_UTF8(*LocalPath), Flags, S_IRUSR | S_IWUSR | S_IRGRP | S_IWGRP) };
		if (Handle.IsValid())
		{
			FFileHandleAndroid* FileHandleAndroid = new FFileHandleAndroid(LocalPath, MoveTemp(Handle));
			if (bAppend)
			{
				FileHandleAndroid->SeekFromEnd(0);
			}
			return FileHandleAndroid;
		}
#if LOG_ANDROID_FILE
		else
		{
			FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FAndroidPlatformFile::OpenWrite('%s') - failed = %s"), Filename, UTF8_TO_TCHAR(strerror(errno)));
		}
#endif
		return nullptr;
	}

	virtual bool DirectoryExists(const TCHAR* Directory) override
	{
		const bool bAllowAssets = false;
		return DirectoryExists(Directory, bAllowAssets);
	}

	virtual bool DirectoryExists(const TCHAR* Directory, bool bAllowAssets) override
	{
		const bool bForceAllowLocal = false;
		return DirectoryExists(Directory, bForceAllowLocal, bAllowAssets);
	}

	bool DirectoryExists(const TCHAR* Directory, bool bForceAllowLocal, bool bAllowAssets)
	{
#if LOG_ANDROID_FILE
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FAndroidPlatformFile::DirectoryExists('%s')"), Directory);
#endif
		FString LocalPath;
		FString AssetPath;
		PathToAndroidPaths(LocalPath, AssetPath, Directory, bForceAllowLocal);

		bool Found = false;
		if (IsLocal(LocalPath))
		{
#if LOG_ANDROID_FILE
			FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FAndroidPlatformFile::DirectoryExists('%s') => Check IsLocal: '%s'"), Directory, *(LocalPath + "/"));
#endif
			struct stat FileInfo;
			if (stat(TCHAR_TO_UTF8(*LocalPath), &FileInfo) != -1)
			{
				Found = S_ISDIR(FileInfo.st_mode);
			}
		}
		else if (IsResource(AssetPath + "/"))
		{
			Found = true;
#if LOG_ANDROID_FILE
			FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FAndroidPlatformFile::DirectoryExists('%s') => Found as resource: '%s'"), Directory, *(AssetPath + "/"));
#endif
		}
		else if (bAllowAssets)
		{
			Found = OpenExistingAssetManagerDirectory(AssetMgr, TCHAR_TO_UTF8(*AssetPath)) != nullptr;
#if LOG_ANDROID_FILE
			if (Found)
			{
				FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FAndroidPlatformFile::DirectoryExists('%s') => Found as asset: '%s'"), Directory, *(AssetPath));
			}
#endif
		}
#if LOG_ANDROID_FILE
		if (Found)
		{
			FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FAndroidPlatformFile::DirectoryExists('%s') => FOUND"), Directory);
		}
		else
		{
			FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FAndroidPlatformFile::DirectoryExists('%s') => NOT"), Directory);
		}
#endif
		return Found;
	}

	// We assert that created dirs are in the local file-system.
	virtual bool CreateDirectory(const TCHAR* Directory) override
	{
		const bool bForceAllowLocal = false;
		return CreateDirectory(Directory, bForceAllowLocal);
	}

	bool CreateDirectory(const TCHAR* Directory, bool bForceAllowLocal)
	{
#if LOG_ANDROID_FILE
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FAndroidPlatformFile::CreateDirectory('%s')"), Directory);
#endif
		FString LocalPath;
		FString AssetPath;
		PathToAndroidPaths(LocalPath, AssetPath, Directory, bForceAllowLocal);
		uint32 mkdirperms = 0755;
#if !UE_BUILD_SHIPPING
		// some devices prevent ADB (shell user) from modifying files.
		// To allow adb shell to modify files we give group users all perms to the new dir.
		mkdirperms = 0775;
#endif
		return (mkdir(TCHAR_TO_UTF8(*LocalPath), mkdirperms) == 0) || (errno == EEXIST);
	}

	// We assert that modifying dirs are in the local file-system.
	virtual bool DeleteDirectory(const TCHAR* Directory) override
	{
		const bool bForceAllowLocal = false;
		return DeleteDirectory(Directory, bForceAllowLocal);
	}

	bool DeleteDirectory(const TCHAR* Directory, bool bForceAllowLocal)
	{
#if LOG_ANDROID_FILE
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FAndroidPlatformFile::DeleteDirectory('%s')"), Directory);
#endif
		FString LocalPath;
		FString AssetPath;
		PathToAndroidPaths(LocalPath, AssetPath, Directory, bForceAllowLocal);

		return rmdir(TCHAR_TO_UTF8(*LocalPath));
	}

	virtual bool IterateDirectory(const TCHAR* Directory, FDirectoryVisitor& Visitor) override
	{
		const bool bAllowAssets = false;
		return IterateDirectory(Directory, Visitor, bAllowAssets);
	}

	virtual bool IterateDirectory(const TCHAR* Directory, FDirectoryVisitor& Visitor, bool bAllowAssets) override
	{
		const bool bForceAllowLocal = false;
		return IterateDirectory(Directory, Visitor, bForceAllowLocal, bAllowAssets);
	}

	bool IterateDirectory(const TCHAR* Directory, FDirectoryVisitor& Visitor, bool bForceAllowLocal, bool bAllowAssets)
	{
		const FString DirectoryStr = Directory;

		auto InternalVisitor = [&](const FString& InLocalPath, struct dirent* InEntry) -> bool
		{
			const FString DirPath = DirectoryStr / UTF8_TO_TCHAR(InEntry->d_name);
			return Visitor.CallShouldVisitAndVisit(*DirPath, InEntry->d_type == DT_DIR);
		};
		
		auto InternalResourceVisitor = [&](const FString& InResourceName, bool IsDirectory) -> bool
		{
			return Visitor.CallShouldVisitAndVisit(*InResourceName, IsDirectory);
		};
		
		auto InternalAssetVisitor = [&](const char* InAssetPath) -> bool
		{
			bool isDirectory = false;
			AAssetDir* subdir = AAssetManager_openDir(AssetMgr, InAssetPath);
			if (nullptr != subdir)
			{
				isDirectory = true;
				AAssetDir_close(subdir);
			}

			return Visitor.CallShouldVisitAndVisit(UTF8_TO_TCHAR(InAssetPath), isDirectory);
		};

		return IterateDirectoryCommon(Directory, InternalVisitor, InternalResourceVisitor, InternalAssetVisitor, bForceAllowLocal, bAllowAssets);
	}

	virtual bool IterateDirectoryStat(const TCHAR* Directory, FDirectoryStatVisitor& Visitor) override
	{
		const bool bAllowAssets = false;
		return IterateDirectoryStat(Directory, Visitor, bAllowAssets);
	}

	virtual bool IterateDirectoryStat(const TCHAR* Directory, FDirectoryStatVisitor& Visitor, bool bAllowAssets) override
	{
		const bool bForceAllowLocal = false;
		return IterateDirectoryStat(Directory, Visitor, bForceAllowLocal, bAllowAssets);
	}

	bool IterateDirectoryStat(const TCHAR* Directory, FDirectoryStatVisitor& Visitor, bool bForceAllowLocal, bool bAllowAssets)
	{
		const FString DirectoryStr = Directory;

		auto InternalVisitor = [&](const FString& InLocalPath, struct dirent* InEntry) -> bool
		{
			const FString DirPath = DirectoryStr / UTF8_TO_TCHAR(InEntry->d_name);

			struct stat FileInfo;
			if (stat(TCHAR_TO_UTF8(*(InLocalPath / UTF8_TO_TCHAR(InEntry->d_name))), &FileInfo) != -1)
			{
				return Visitor.CallShouldVisitAndVisit(*DirPath, AndroidStatToUEFileData(FileInfo));
			}

			return true;
		};
		
		auto InternalResourceVisitor = [&](const FString& InResourceName, bool IsDir) -> bool
		{
			return Visitor.CallShouldVisitAndVisit(
				*InResourceName, 
				FFileStatData(
					FDateTime::MinValue(),						// CreationTime
					FDateTime::MinValue(),						// AccessTime
					FDateTime::MinValue(),						// ModificationTime
					IsDir ? -1 : ZipResource.GetEntryLength(InResourceName),	// FileSize
					IsDir,										// bIsDirectory
					true										// bIsReadOnly
					)
				);
		};
		
		auto InternalAssetVisitor = [&](const char* InAssetPath) -> bool
		{
			bool isDirectory = false;
			AAssetDir* subdir = AAssetManager_openDir(AssetMgr, InAssetPath);
			if (nullptr != subdir)
			{
				isDirectory = true;
				AAssetDir_close(subdir);
			}

			int64 FileSize = -1;
			if (!isDirectory)
			{
				AAsset* file = AAssetManager_open(AssetMgr, InAssetPath, AASSET_MODE_UNKNOWN);
				FileSize = AAsset_getLength(file);
				AAssetDir_close(subdir);
			}

			return Visitor.CallShouldVisitAndVisit(
				UTF8_TO_TCHAR(InAssetPath), 
				FFileStatData(
					FDateTime::MinValue(),	// CreationTime
					FDateTime::MinValue(),	// AccessTime
					FDateTime::MinValue(),	// ModificationTime
					FileSize,				// FileSize
					isDirectory,			// bIsDirectory
					true					// bIsReadOnly
					)
				);
		};

		return IterateDirectoryCommon(Directory, InternalVisitor, InternalResourceVisitor, InternalAssetVisitor, bForceAllowLocal, bAllowAssets);
	}

	bool IterateDirectoryCommon(const TCHAR* Directory, const TFunctionRef<bool(const FString&, struct dirent*)>& Visitor, const TFunctionRef<bool(const FString&, bool)>& ResourceVisitor, const TFunctionRef<bool(const char*)>& AssetVisitor, bool bForceAllowLocal, bool bAllowAssets)
	{
#if LOG_ANDROID_FILE
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FAndroidPlatformFile::IterateDirectory('%s')"), Directory);
#endif
		FString LocalPath;
		FString AssetPath;
		PathToAndroidPaths(LocalPath, AssetPath, Directory, bForceAllowLocal);

		if (IsLocal(LocalPath))
		{
			DIR* Handle = opendir(TCHAR_TO_UTF8(*LocalPath));
			if (Handle)
			{
				bool Result = true;
				struct dirent *Entry;
				while ((Entry = readdir(Handle)) != nullptr && Result == true)
				{
					if (FCString::Strcmp(UTF8_TO_TCHAR(Entry->d_name), TEXT(".")) && FCString::Strcmp(UTF8_TO_TCHAR(Entry->d_name), TEXT("..")))
					{
#if LOG_ANDROID_FILE
						FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FAndroidPlatformFile::IterateDirectory('%s').. LOCAL Visit: '%s'"), Directory, *(FString(Directory) / UTF8_TO_TCHAR(Entry->d_name)));
#endif
						Result = Visitor(LocalPath, Entry);
					}
				}
				closedir(Handle);
				return Result;
			}
		}
		else if (IsResource(AssetPath))
		{
			FZipUnionFile::Directory ResourceDir = ZipResource.OpenDirectory(AssetPath);
			bool Result = true;
			while (Result && ResourceDir.Next())
			{
#if LOG_ANDROID_FILE
				FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FAndroidPlatformFile::IterateDirectory('%s').. RESOURCE Visit: '%s'"), Directory, *ResourceDir.Current.Key());
#endif
				FString ResourcePath = ResourceDir.Current.Key();
				bool IsDirectory = ResourceDir.Current.Value()->IsDirectory;
				if (IsDirectory && ResourcePath.EndsWith(TEXT("/"), ESearchCase::CaseSensitive))
				{
					ResourcePath.GetCharArray()[ResourcePath.Len() - 1] = 0;
					ResourcePath.TrimToNullTerminator();
				}
				Result = ResourceVisitor(ResourcePath, IsDirectory);
			}
			return Result;
		}
		else if (IsResource(AssetPath + "/"))
		{
			FZipUnionFile::Directory ResourceDir = ZipResource.OpenDirectory(AssetPath + "/");
			bool Result = true;
			while (Result && ResourceDir.Next())
			{
#if LOG_ANDROID_FILE
				FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FAndroidPlatformFile::IterateDirectory('%s').. RESOURCE/ Visit: '%s'"), Directory, *ResourceDir.Current.Key());
#endif
				FString ResourcePath = ResourceDir.Current.Key();
				bool IsDirectory = ResourceDir.Current.Value()->IsDirectory;
				if (IsDirectory && ResourcePath.EndsWith(TEXT("/"), ESearchCase::CaseSensitive))
				{
					ResourcePath.GetCharArray()[ResourcePath.Len() - 1] = 0;
					ResourcePath.TrimToNullTerminator();
				}
				Result = ResourceVisitor(ResourcePath, IsDirectory);
			}
			return Result;
		}
		else if (bAllowAssets)
		{
			AAssetDir * dir = AAssetManager_openDir(AssetMgr, TCHAR_TO_UTF8(*AssetPath));
			if (LIKELY(nullptr != dir))
			{
				bool Result = true;
				const char * fileName = nullptr;
				while ((fileName = AAssetDir_getNextFileName(dir)) != nullptr && Result == true)
				{
#if LOG_ANDROID_FILE
					FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FAndroidPlatformFile::IterateDirectory('%s').. ASSET Visit: '%s'"), Directory, UTF8_TO_TCHAR(fileName));
#endif
					Result = AssetVisitor(fileName);
				}
				AAssetDir_close(dir);
				return Result;
			}
		}
		return false;
	}

#if USE_ANDROID_JNI
	virtual jobject GetAssetManager() override
	{
		return AndroidJNI_GetJavaAssetManager();
	}
#endif

	virtual bool IsAsset(const TCHAR* Filename) override
	{
#if LOG_ANDROID_FILE
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FAndroidPlatformFile::FileIsAsset('%s')"), Filename);
#endif
		FString LocalPath;
		FString AssetPath;
		const bool bForceAllowLocal = true;
		PathToAndroidPaths(LocalPath, AssetPath, Filename, bForceAllowLocal);

		if (IsLocal(LocalPath))
		{
			return false;
		}
		else if (IsResource(AssetPath))
		{
			return ZipResource.GetEntryAsset(AssetPath) != nullptr;
		}
		else if (IsAsset(AssetPath))
		{
			return true;
		}
		return false;
	}

	virtual int64 FileStartOffset(const TCHAR* Filename) override
	{
#if LOG_ANDROID_FILE
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FAndroidPlatformFile::FileStartOffset('%s')"), Filename);
#endif
		FString LocalPath;
		FString AssetPath;
		const bool bForceAllowLocal = true;
		PathToAndroidPaths(LocalPath, AssetPath, Filename, bForceAllowLocal);

		if (IsLocal(LocalPath))
		{
			return 0;
		}
		else if (IsResource(AssetPath))
		{
			return ZipResource.GetEntry(AssetPath).File->Start;
		}
		else if (IsAsset(AssetPath))
		{
			AAsset * file = AAssetManager_open(AssetMgr, TCHAR_TO_UTF8(*AssetPath), AASSET_MODE_UNKNOWN);
			if (nullptr != file)
			{
				off_t start = -1;
				off_t length = -1;
				int handle = AAsset_openFileDescriptor(file, &start, &length);
				if (handle != -1)
				{
					close(handle);
				}
				AAsset_close(file);
				return start;
			}
		}
		return -1;
	}

	virtual FString FileRootPath(const TCHAR* Filename) override
	{
#if LOG_ANDROID_FILE
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FAndroidPlatformFile::FileRootPath('%s')"), Filename);
#endif
		FString LocalPath;
		FString AssetPath;
		const bool bForceAllowLocal = true;
		PathToAndroidPaths(LocalPath, AssetPath, Filename, bForceAllowLocal);

		if (IsLocal(LocalPath))
		{
			return LocalPath;
		}
		else if (IsResource(AssetPath))
		{
			return ZipResource.GetEntryRootPath(AssetPath);
		}
		else if (IsAsset(AssetPath))
		{
			return AssetPath;
		}
		return "";
	}


private:
	FString NormalizePath(const TCHAR* Path)
	{
		FString Result(Path);
		Result.ReplaceInline(TEXT("\\"), TEXT("/"), ESearchCase::CaseSensitive);
		// This replacement addresses a "bug" where some callers
		// pass in paths that are badly composed with multiple
		// subdir separators.
		Result.ReplaceInline(TEXT("//"), TEXT("/"), ESearchCase::CaseSensitive);
		// Remove redundant current-dir references.
		Result.ReplaceInline(TEXT("/./"), TEXT("/"), ESearchCase::CaseSensitive);
		return Result;
	}

	void PathToAndroidPaths(FString & LocalPath, FString & AssetPath, const TCHAR* Path, bool bForceAllowLocal)
	{
		LocalPath.Empty();
		AssetPath.Empty();

		FString AndroidPath = NormalizePath(Path);
#if LOG_ANDROID_FILE
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FAndroidPlatformFile::PathToAndroidPaths('%s') => AndroidPath = '%s'"), Path, *AndroidPath);
#endif
		if (!AndroidPath.IsEmpty())
		{
			// We filter out non-permitted local paths here when ANDROID_DISALLOW_LOCAL_FILESYSTEM is set.
			// There is a limited set of paths we always allow local file access to, such as direct font access,
			// and bForceAllowLocal bypasses the restriction as necessary to mount OBB files.
			if (
#if ANDROID_DISALLOW_LOCAL_FILESYSTEM
				(bForceAllowLocal && AndroidPath.StartsWith(TEXT("/"), ESearchCase::CaseSensitive)) ||
#else
				AndroidPath.StartsWith(TEXT("/"), ESearchCase::CaseSensitive) ||
#endif
				AndroidPath.StartsWith(GFontPathBase) ||
				AndroidPath.StartsWith(TEXT("/system/etc/")) ||
				AndroidPath.StartsWith(GInternalFilePath.Left(AndroidPath.Len())) ||
				AndroidPath.StartsWith(GExternalFilePath.Left(AndroidPath.Len())))
			{
				// Absolute paths are only local.
				LocalPath = AndroidPath;
				AssetPath = AndroidPath;
			}
			else
			{
				while (AndroidPath.StartsWith(TEXT("../"), ESearchCase::CaseSensitive))
				{
					AndroidPath.RightChopInline(3, EAllowShrinking::No);
				}
				AndroidPath.ReplaceInline(FPlatformProcess::BaseDir(), TEXT(""));
				if (AndroidPath.Equals(TEXT(".."), ESearchCase::CaseSensitive))
				{
					AndroidPath = TEXT("");
				}

				// Local filepaths are directly in the deployment directory.
				static FString BasePath = GetFileBasePath();
				LocalPath = BasePath + AndroidPath;

				// Asset paths are relative to the base directory.
				AssetPath = AndroidPath;
			}
		}

#if LOG_ANDROID_FILE
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FAndroidPlatformFile::PathToAndroidPaths('%s') => LocalPath = '%s'"), Path, *LocalPath);
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FAndroidPlatformFile::PathToAndroidPaths('%s') => AssetPath = '%s'"), Path, *AssetPath);
#endif
	}

	bool IsLocal(const FString & LocalPath)
	{
		return !LocalPath.IsEmpty() && access(TCHAR_TO_UTF8(*LocalPath), F_OK) == 0;
	}

	bool IsAsset(const FString & AssetPath)
	{
		AAsset * file = AAssetManager_open(AssetMgr, TCHAR_TO_UTF8(*AssetPath), AASSET_MODE_UNKNOWN);
		if (nullptr != file)
		{
			AAsset_close(file);
			return true;
		}
		return false;
	}

	bool IsResource(const FString & ResourcePath)
	{
		return ZipResource.HasEntry(ResourcePath);
	}

	class FMountOBBVisitor : public IPlatformFile::FDirectoryVisitor
	{
		FAndroidPlatformFile & AndroidPlatformFile;

	public:
		FMountOBBVisitor(FAndroidPlatformFile & APF)
			: AndroidPlatformFile(APF)
		{
		}

		virtual ~FMountOBBVisitor()
		{
		}

		virtual bool Visit(const TCHAR* FilenameOrDirectory, bool bIsDirectory) override
		{
			if (FString(FilenameOrDirectory).EndsWith(TEXT(".obb")) ||
				FString(FilenameOrDirectory).EndsWith(TEXT(".obb.png")))
			{
				// It's and OBB (actually a ZIP) so we fake mount it.
				AndroidPlatformFile.MountOBB(FilenameOrDirectory);
			}
			return true;
		}
	};

	void MountOBB(const TCHAR* Filename)
	{
		const bool bForceAllowLocal = true;
		const bool bAllowWrite = false;

		FFileHandleAndroid* File = static_cast<FFileHandleAndroid*>(OpenReadInternal(Filename, bForceAllowLocal, bAllowWrite));
		check(nullptr != File);
		ZipResource.AddPatchFile(MakeShareable(File));
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Mounted OBB '%s'"), Filename);
	}

	AAssetManager* AssetMgr;
	FZipUnionFile ZipResource;
};

IPlatformFile& IPlatformFile::GetPlatformPhysical()
{
	return FAndroidPlatformFile::GetPlatformPhysical();
}

IAndroidPlatformFile& IAndroidPlatformFile::GetPlatformPhysical()
{
	return FAndroidPlatformFile::GetPlatformPhysical();
}

const FString* IAndroidPlatformFile::GetOverrideLogDirectory()
{
	return FAndroidPlatformFile::GetOverrideLogDirectory();
}

FString IAndroidPlatformFile::ConvertToAbsolutePathForExternalAppForRead(const TCHAR* Filename)
{
	return AndroidRelativeToAbsolutePath(false, Filename);
}

FString IAndroidPlatformFile::ConvertToAbsolutePathForExternalAppForWrite(const TCHAR* Filename)
{
	return AndroidRelativeToAbsolutePath(false, Filename);
}

#endif
