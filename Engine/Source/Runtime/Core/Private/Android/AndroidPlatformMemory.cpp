// Copyright Epic Games, Inc. All Rights Reserved.

#include "Android/AndroidPlatformMemory.h"
#include "Android/AndroidHeapProfiling.h"
#include "Async/UniqueLock.h"
#include "HAL/LowLevelMemTracker.h"
#include "HAL/MallocBinned.h"
#include "HAL/MallocBinned2.h"
#include "HAL/MallocBinned3.h"
#include "HAL/MallocMimalloc.h"
#include "HAL/MallocAnsi.h"
#include "HAL/PlatformMutex.h"
#include "unistd.h"
#include <jni.h>
#include <sys/sysinfo.h>
#include <sys/mman.h>
#if !UE_BUILD_SHIPPING
#include <linux/prctl.h> 
#include <sys/prctl.h>
#endif

#include "Android/AndroidPlatformCrashContext.h"
#include "Async/TaskGraphInterfaces.h"
#include "Async/Async.h"
#include "Tasks/Task.h"

#define JNI_CURRENT_VERSION JNI_VERSION_1_6
extern JavaVM* GJavaVM;

static int32 GAndroidAddSwapToTotalPhysical = 1;
static FAutoConsoleVariableRef CVarAddSwapToTotalPhysical(
	TEXT("android.AddSwapToTotalPhysical"),
	GAndroidAddSwapToTotalPhysical,
	TEXT("When non zero, Total physical memory reporting will also include swap memory. (default)"),
	ECVF_Default
);

static int64 GetNativeHeapAllocatedSize()
{
	int64 AllocatedSize = 0;
#if 0 // TODO: this works but sometimes crashes?
	JNIEnv* Env = NULL;
	GJavaVM->GetEnv((void **)&Env, JNI_CURRENT_VERSION);
	jint AttachThreadResult = GJavaVM->AttachCurrentThread(&Env, NULL);

	if(AttachThreadResult != JNI_ERR)
	{
		jclass Class = Env->FindClass("android/os/Debug");
		if (Class)
		{
			jmethodID MethodID = Env->GetStaticMethodID(Class, "getNativeHeapAllocatedSize", "()J");
			if (MethodID)
			{
				AllocatedSize = Env->CallStaticLongMethod(Class, MethodID);
			}
		}
	}
#endif
	return AllocatedSize;
}

/** Controls growth of pools - see PooledVirtualMemoryAllocator.cpp */
extern float GVMAPoolScale;

#if !UE_BUILD_SHIPPING
#ifndef PR_SET_VMA
#define PR_SET_VMA 0x53564d41
#endif
#ifndef PR_SET_VMA_ANON_NAME
#define PR_SET_VMA_ANON_NAME 0
#endif
static inline void SetMmapName(const void* Ptr, const size_t Size, const char* Name)
{
	prctl(PR_SET_VMA, PR_SET_VMA_ANON_NAME, Ptr, Size, Name);
}
#else
static inline void SetMmapName(const void*, const size_t, const char*) {}
#endif

void FAndroidPlatformMemory::Init()
{
	// Only allow this method to be called once
	{
		static bool bInitDone = false;
		if (bInitDone)
			return;
		bInitDone = true;
	}

	FGenericPlatformMemory::Init();

	const FPlatformMemoryConstants& MemoryConstants = FPlatformMemory::GetConstants();
	FPlatformMemoryStats MemoryStats = GetStats();
	UE_LOGF(LogInit, Log, "Memory total: Physical=%.2fMB (%dGB approx) Available=%.2fMB PageSize=%.1fKB", 
		float((double)MemoryConstants.TotalPhysical/1024.0/1024.0),
		MemoryConstants.TotalPhysicalGB, 
		float((double)MemoryStats.AvailablePhysical/1024.0/1024.0),
		float((double)MemoryConstants.PageSize/1024.0)
		);
	UE_LOGF(LogInit, Log, " - VirtualMemoryAllocator pools will grow at scale %g", GVMAPoolScale);
}

extern void (*GMemoryWarningHandler)(const FGenericMemoryWarningContext& Context);

namespace AndroidPlatformMemory
{
	/**
	* @brief Returns value in bytes from a status line
	* @param Line in format "Blah:  10000 kB" - needs to be writable as it will modify it
	* @return value in bytes (10240000, i.e. 10000 * 1024 for the above example)
	*/
	uint64 GetBytesFromStatusLine(char * Line)
	{
		check(Line);
		int Len = strlen(Line);

		// Len should be long enough to hold at least " kB\n"
		const int kSuffixLength = 4;	// " kB\n"
		if (Len <= kSuffixLength)
		{
			return 0;
		}

		// let's check that this is indeed "kB"
		char * Suffix = &Line[Len - kSuffixLength];
		if (strcmp(Suffix, " kB\n") != 0)
		{
			// Linux the kernel changed the format, huh?
			return 0;
		}

		// kill the kB
		*Suffix = 0;

		// find the beginning of the number
		for (const char * NumberBegin = Suffix; NumberBegin >= Line; --NumberBegin)
		{
			if (*NumberBegin == ' ')
			{
				return static_cast< uint64 >(atol(NumberBegin + 1)) * 1024ULL;
			}
		}

		// we were unable to find whitespace in front of the number
		return 0;
	}

	static FAndroidMemoryWarningContext GAndroidMemoryWarningContext;
	static void SendMemoryWarningContext()
	{
		if (FTaskGraphInterface::IsRunning())
		{
			// Run on game thread to avoid mem handler callback getting confused.
			AsyncTask(ENamedThreads::GameThread, [AndroidMemoryWarningContext = GAndroidMemoryWarningContext]()
				{
					if (GMemoryWarningHandler)
					{
						// note that we may also call this when recovering from low memory conditions. (i.e. not in low memory state.)
						GMemoryWarningHandler(AndroidMemoryWarningContext);
					}
				});
		}
		else
		{
			const FAndroidMemoryWarningContext& Context = GAndroidMemoryWarningContext;
			UE_LOGF(LogAndroid, Warning, "Not calling memory warning handler, received too early. Last Trim Memory State: %d", (int)Context.LastTrimMemoryState);
		}
	}
}

extern int32 AndroidThunkCpp_GetMetaDataInt(const FString& Key);

// useful for debugging, this triggers a traversal of smaps and can take 100s of ms.
static bool GetPss(uint64& PSSOUT, uint64& SwapPSSOUT)
{
	PSSOUT = 0;
	SwapPSSOUT = 0;
	int FieldsSetSuccessfully = 0;
	// again /proc "API" :/
	if (FILE* SmapsRollup = fopen("/proc/self/smaps_rollup", "r"))
	{
		do
		{
			char LineBuffer[256] = { 0 };
			char* Line = fgets(LineBuffer, UE_ARRAY_COUNT(LineBuffer), SmapsRollup);
			if (Line == nullptr)
			{
				break;	// eof or an error
			}
			if (strstr(Line, "SwapPss:") == Line)
			{
				SwapPSSOUT = AndroidPlatformMemory::GetBytesFromStatusLine(Line);
				++FieldsSetSuccessfully;
			}
			else if (strstr(Line, "Pss:") == Line)
			{
				PSSOUT = AndroidPlatformMemory::GetBytesFromStatusLine(Line);
				++FieldsSetSuccessfully;
			}
		} while (FieldsSetSuccessfully < 2);
		fclose(SmapsRollup);
	}
	return FieldsSetSuccessfully == 2;
}

FPlatformMemoryStats FAndroidPlatformMemory::GetStats()
{
	FPlatformMemoryStats MemoryStats;	// will init from constants

										// open to all kind of overflows, thanks to Linux approach of exposing system stats via /proc and lack of proper C API
										// And no, sysinfo() isn't useful for this (cannot get the same value for MemAvailable through it for example).

	if (FILE* FileGlobalMemStats = fopen("/proc/meminfo", "r"))
	{
		int FieldsSetSuccessfully = 0;
		uint64 MemFree = 0, Cached = 0;
		do
		{
			char LineBuffer[256] = { 0 };
			char *Line = fgets(LineBuffer, UE_ARRAY_COUNT(LineBuffer), FileGlobalMemStats);
			if (Line == nullptr)
			{
				break;	// eof or an error
			}

			// if we have MemAvailable, favor that (see http://git.kernel.org/cgit/linux/kernel/git/torvalds/linux.git/commit/?id=34e431b0ae398fc54ea69ff85ec700722c9da773)
			if (strstr(Line, "MemAvailable:") == Line)
			{
				MemoryStats.AvailablePhysical = AndroidPlatformMemory::GetBytesFromStatusLine(Line);
				++FieldsSetSuccessfully;
			}
			else if (strstr(Line, "SwapFree:") == Line)
			{
				MemoryStats.AvailableVirtual = AndroidPlatformMemory::GetBytesFromStatusLine(Line);
				++FieldsSetSuccessfully;
			}
			else if (strstr(Line, "MemFree:") == Line)
			{
				MemFree = AndroidPlatformMemory::GetBytesFromStatusLine(Line);
				++FieldsSetSuccessfully;
			}
			else if (strstr(Line, "Cached:") == Line)
			{
				Cached = AndroidPlatformMemory::GetBytesFromStatusLine(Line);
				++FieldsSetSuccessfully;
			}
		} while (FieldsSetSuccessfully < 4);

		// if we didn't have MemAvailable (kernels < 3.14 or CentOS 6.x), use free + cached as a (bad) approximation
		if (MemoryStats.AvailablePhysical == 0)
		{
			MemoryStats.AvailablePhysical = FMath::Min(MemFree + Cached, MemoryStats.TotalPhysical);
		}

		fclose(FileGlobalMemStats);
	}

	// again /proc "API" :/
	if (FILE* ProcMemStats = fopen("/proc/self/status", "r"))
	{
		int FieldsSetSuccessfully = 0;
		do
		{
			char LineBuffer[256] = { 0 };
			char *Line = fgets(LineBuffer, UE_ARRAY_COUNT(LineBuffer), ProcMemStats);
			if (Line == nullptr)
			{
				break;	// eof or an error
			}

			if (strstr(Line, "VmPeak:") == Line)
			{
				MemoryStats.PeakUsedVirtual = AndroidPlatformMemory::GetBytesFromStatusLine(Line);
				++FieldsSetSuccessfully;
			}
			else if (strstr(Line, "VmSize:") == Line)
			{
				MemoryStats.UsedVirtual = AndroidPlatformMemory::GetBytesFromStatusLine(Line);
				++FieldsSetSuccessfully;
			}
			else if (strstr(Line, "VmHWM:") == Line)
			{
				MemoryStats.PeakUsedPhysical = AndroidPlatformMemory::GetBytesFromStatusLine(Line);
				++FieldsSetSuccessfully;
			}
			else if (strstr(Line, "VmRSS:") == Line)
			{
				MemoryStats.UsedPhysical = AndroidPlatformMemory::GetBytesFromStatusLine(Line);
				MemoryStats.VMRss = MemoryStats.UsedPhysical;
				++FieldsSetSuccessfully;
			}
			else if (strstr(Line, "RssFile:") == Line)
			{
				MemoryStats.RssFile = AndroidPlatformMemory::GetBytesFromStatusLine(Line);
				++FieldsSetSuccessfully;
			}
			else if (strstr(Line, "VmSwap:") == Line)
			{
				MemoryStats.VMSwap = AndroidPlatformMemory::GetBytesFromStatusLine(Line);
				++FieldsSetSuccessfully;
			}
		} while (FieldsSetSuccessfully < 6);

		fclose(ProcMemStats);
	}

	if (GAndroidAddSwapToTotalPhysical)
	{
		MemoryStats.UsedPhysical += MemoryStats.VMSwap;
	}


	// sanitize stats as sometimes peak < used for some reason
	MemoryStats.PeakUsedVirtual = FMath::Max(MemoryStats.PeakUsedVirtual, MemoryStats.UsedVirtual);
	MemoryStats.PeakUsedPhysical = FMath::Max(MemoryStats.PeakUsedPhysical, MemoryStats.UsedPhysical);

	// get this value from Java instead (DO NOT INTEGRATE at this time) - skip this if JavaVM not set up yet!
#if USE_ANDROID_JNI
	// note: Android 10 places impractical limits on the frequency of calls to getProcessMemoryInfo, revert to VmRSS for OS10+
	if (GJavaVM && FAndroidMisc::GetAndroidBuildVersion() < 29) 
	{
		MemoryStats.UsedPhysical = static_cast<uint64>(AndroidThunkCpp_GetMetaDataInt(TEXT("unreal.getUsedMemory"))) * 1024ULL;
	}
#endif

	return MemoryStats;
}

FGenericPlatformMemoryStats::EMemoryPressureStatus FPlatformMemoryStats::GetMemoryPressureStatus() const
{
	// convert Android's TRIM status to FGenericPlatformMemoryStats::EMemoryPressureStatus.
	auto AndroidTRIMToMemPressureStatus = [](FAndroidPlatformMemory::ETrimValues LastTrimMemoryState)
	{
		switch (LastTrimMemoryState)
		{
		case FAndroidPlatformMemory::ETrimValues::Running_Critical:
			return FGenericPlatformMemoryStats::EMemoryPressureStatus::Critical;
		case FAndroidPlatformMemory::ETrimValues::Unknown:
			return FGenericPlatformMemoryStats::EMemoryPressureStatus::Unknown;
		case FAndroidPlatformMemory::ETrimValues::Complete:
		case FAndroidPlatformMemory::ETrimValues::Running_Low:
			return FGenericPlatformMemoryStats::EMemoryPressureStatus::Warning;
		case FAndroidPlatformMemory::ETrimValues::Moderate:
		case FAndroidPlatformMemory::ETrimValues::Background:
		case FAndroidPlatformMemory::ETrimValues::UI_Hidden:
		case FAndroidPlatformMemory::ETrimValues::Running_Moderate:
		default:
			return FGenericPlatformMemoryStats::EMemoryPressureStatus::Nominal;
		}
	};

	return AndroidTRIMToMemPressureStatus(AndroidPlatformMemory::GAndroidMemoryWarningContext.LastTrimMemoryState);
}

uint64 FAndroidPlatformMemory::GetMemoryUsedFast()
{
	// get this value from Java instead (DO NOT INTEGRATE at this time) - skip this if JavaVM not set up yet!
#if USE_ANDROID_JNI
	// note: Android 10 places impractical limits on the frequency of calls to getProcessMemoryInfo, revert to VmRSS for OS10+
	if (GJavaVM && FAndroidMisc::GetAndroidBuildVersion() < 29) 
	{
		return static_cast<uint64>(AndroidThunkCpp_GetMetaDataInt(TEXT("unreal.getUsedMemory"))) * 1024ULL;
	}
#endif

	uint64 VMRSS = 0;
	uint64 VMSwap = 0;
	// minimal code to get Used memory
	if (FILE* ProcMemStats = fopen("/proc/self/status", "r"))
	{
		int LinesToFind = GAndroidAddSwapToTotalPhysical ? 2 : 1;
		while (1)
		{
			char LineBuffer[256] = { 0 };
			char *Line = fgets(LineBuffer, UE_ARRAY_COUNT(LineBuffer), ProcMemStats);
			if (Line == nullptr)
			{
				break;	// eof or an error
			}
			else if (strstr(Line, "VmRSS:") == Line)
			{
				VMRSS = AndroidPlatformMemory::GetBytesFromStatusLine(Line);
				LinesToFind--;
			}
			else if (GAndroidAddSwapToTotalPhysical && strstr(Line, "VmSwap:") == Line)
			{
				VMSwap = AndroidPlatformMemory::GetBytesFromStatusLine(Line);
				LinesToFind--;
			}

			if (LinesToFind == 0)
			{
				break;
			}
		} 
		fclose(ProcMemStats);
	}

	return VMRSS + VMSwap;
}

// Called from JNI when trim messages are received.
void FAndroidPlatformMemory::UpdateOSMemoryStatus(EOSMemoryStatusCategory OSMemoryStatusCategory, int Value)
{
	switch (OSMemoryStatusCategory)
	{
		case EOSMemoryStatusCategory::OSTrim:
			AndroidPlatformMemory::GAndroidMemoryWarningContext.LastTrimMemoryState = (FAndroidPlatformMemory::ETrimValues)Value;
			break;
		default:
			checkNoEntry();
	}

	AndroidPlatformMemory::SendMemoryWarningContext();
}

const FPlatformMemoryConstants& FAndroidPlatformMemory::GetConstants()
{
	static FPlatformMemoryConstants MemoryConstants;

	if (MemoryConstants.TotalPhysical == 0)
	{
		// Gather platform memory stats.
		struct sysinfo SysInfo;
		unsigned long long MaxPhysicalRAMBytes = 0;
		unsigned long long MaxVirtualRAMBytes = 0;

		if (0 == sysinfo(&SysInfo))
		{
			MaxPhysicalRAMBytes = static_cast< unsigned long long >(SysInfo.mem_unit) * static_cast< unsigned long long >(SysInfo.totalram);
			MaxVirtualRAMBytes
				= MaxPhysicalRAMBytes
				+ static_cast< unsigned long long >(SysInfo.mem_unit) * static_cast< unsigned long long >(SysInfo.totalswap);
		}

		MemoryConstants.TotalPhysical = MaxPhysicalRAMBytes;
		MemoryConstants.TotalVirtual = MaxVirtualRAMBytes;
		MemoryConstants.TotalPhysicalGB = (MemoryConstants.TotalPhysical + 1024 * 1024 * 1024 - 1) / 1024 / 1024 / 1024;
		MemoryConstants.PageSize = sysconf(_SC_PAGESIZE);
		MemoryConstants.BinnedPageSize = FMath::Max((SIZE_T)65536, MemoryConstants.PageSize);
		MemoryConstants.BinnedAllocationGranularity = MemoryConstants.PageSize;
		MemoryConstants.OsAllocationGranularity = MemoryConstants.PageSize;
		MemoryConstants.AddressLimit = FPlatformMath::RoundUpToPowerOfTwo64(MemoryConstants.TotalPhysical);
	}

	return MemoryConstants;
}

EPlatformMemorySizeBucket FAndroidPlatformMemory::GetMemorySizeBucket()
{
	// @todo android - if running without the extensions for texture streaming, we will load all of the textures
	// so we better look like a low memory device
	return FGenericPlatformMemory::GetMemorySizeBucket();
}

FMalloc* FAndroidPlatformMemory::BaseAllocator()
{
	static FMalloc* Instance = nullptr;
	if (Instance != nullptr)
	{
		return Instance;
	}

	FPlatformMemoryStats Stats = FAndroidPlatformMemory::GetStats();
	FPlatformMemory::ProgramSize = Stats.UsedPhysical;

#if FORCE_ANSI_ALLOCATOR
	return (Instance = new FMallocAnsi());
#endif

#if	!ENABLE_PGO_PROFILE
	const bool bUseMimalloc = FPlatformMisc::GetConfigRulesVariableAsUtf8("malloctype").Equals("mimalloc");
	if (bUseMimalloc)
	{
		return (Instance = new FMallocMimalloc());
	}
#endif

	const bool bHeapProfilingSupported = AndroidHeapProfiling::Init();

#if USE_MALLOC_BINNED3
	if (bHeapProfilingSupported)
	{
		return (Instance = new FMallocProfilingProxy<FMallocBinned3>());
	}
	return (Instance = new FMallocBinned3());
#elif USE_MALLOC_BINNED2
	if (bHeapProfilingSupported)
	{
		return (Instance = new FMallocProfilingProxy<FMallocBinned2>());
	}
	return (Instance = new FMallocBinned2());
#else
	const FPlatformMemoryConstants& MemoryConstants = FPlatformMemory::GetConstants();
	// 1 << FMath::CeilLogTwo(MemoryConstants.TotalPhysical) should really be FMath::RoundUpToPowerOfTwo,
	// but that overflows to 0 when MemoryConstants.TotalPhysical is close to 4GB, since CeilLogTwo returns 32
	// this then causes the MemoryLimit to be 0 and crashing the app
	uint64 MemoryLimit = FMath::Min<uint64>(uint64(1) << FMath::CeilLogTwo(MemoryConstants.TotalPhysical), 0x100000000);

	// todo: Verify MallocBinned2 on 32bit
	// [RCL] 2017-03-06 FIXME: perhaps BinnedPageSize should be used here, but leaving this change to the Android platform owner.
	return (Instance = new FMallocBinned(MemoryConstants.PageSize, MemoryLimit));
#endif
}


void* FAndroidPlatformMemory::BinnedAllocFromOS(SIZE_T Size)
{
	void* Ptr;
	if (USE_MALLOC_BINNED2)
	{
		static UE::FPlatformRecursiveMutex Mutex;
		UE::TUniqueLock Lock(Mutex);

		const FPlatformMemoryConstants& MemoryConstants = FPlatformMemory::GetConstants();

		static uint8* Base = nullptr;
		static uint8* End = nullptr;

		static const SIZE_T MinAllocSize = 4 * 1024 * 1024; // we will allocate chunks of 4MB, that means the amount we will need to unmap, assuming a lot of 64k blocks, will be small.

		if (End - Base < Size)
		{
			if (Base)
			{
				if (Base < End)
				{
					if (munmap(Base, End - Base) != 0)
					{
						const int ErrNo = errno;
						UE_LOGF(LogHAL, Fatal, "munmap (for trim) (addr=%p, len=%llu) failed with errno = %d (%ls)", Base, End - Base,
							ErrNo, StringCast< TCHAR >(strerror(ErrNo)).Get());
					}
				}
				Base = nullptr;
				End = nullptr;
			}

			SIZE_T SizeToAlloc = FMath::Max<SIZE_T>(MinAllocSize, Align(Size, MemoryConstants.PageSize) + MemoryConstants.BinnedPageSize);
			uint8* UnalignedBase = (uint8*)mmap(nullptr, SizeToAlloc, PROT_READ | PROT_WRITE, MAP_PRIVATE | MAP_ANON, -1, 0);
			SetMmapName(UnalignedBase, SizeToAlloc, "UE");
			End = UnalignedBase + SizeToAlloc;
			Base = Align(UnalignedBase, MemoryConstants.BinnedPageSize);

			if (Base > UnalignedBase)
			{
				if (munmap(UnalignedBase, Base - UnalignedBase) != 0)
				{
					const int ErrNo = errno;
					UE_LOGF(LogHAL, Fatal, "munmap (for align) (addr=%p, len=%llu) failed with errno = %d (%ls)", UnalignedBase, Base - UnalignedBase,
						ErrNo, StringCast< TCHAR >(strerror(ErrNo)).Get());
				}
			}


		}
		Ptr = Base;
		uint8* UnalignedBase = Align(Base + Size, MemoryConstants.PageSize);
		Base = Align(UnalignedBase, MemoryConstants.BinnedPageSize);

		if (Base > End)
		{
			Base = End;
		}

		if (Base > UnalignedBase)
		{
			if (munmap(UnalignedBase, Base - UnalignedBase) != 0)
			{
				const int ErrNo = errno;
				UE_LOGF(LogHAL, Fatal, "munmap (for tail align) (addr=%p, len=%llu) failed with errno = %d (%ls)", UnalignedBase, Base - UnalignedBase,
					ErrNo, StringCast< TCHAR >(strerror(ErrNo)).Get());
			}
		}
	}
	else
	{
		Ptr = mmap(nullptr, Size, PROT_READ | PROT_WRITE, MAP_PRIVATE | MAP_ANON, -1, 0);
		SetMmapName(Ptr, Size, "UE");
	}

	LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelAlloc(ELLMTracker::Platform, Ptr, Size));
	return Ptr;
}

void FAndroidPlatformMemory::BinnedFreeToOS(void* Ptr, SIZE_T Size)
{
	LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelFree(ELLMTracker::Platform, Ptr));
	if (munmap(Ptr, Size) != 0)
	{
		const int ErrNo = errno;
		UE_LOGF(LogHAL, Fatal, "munmap(addr=%p, len=%llu) failed with errno = %d (%ls)", Ptr, Size,
			ErrNo, StringCast< TCHAR >(strerror(ErrNo)).Get());
	}
}

size_t FAndroidPlatformMemory::FPlatformVirtualMemoryBlock::GetVirtualSizeAlignment()
{
	static SIZE_T OSPageSize = FPlatformMemory::GetConstants().PageSize;
	return OSPageSize;
}

size_t FAndroidPlatformMemory::FPlatformVirtualMemoryBlock::GetCommitAlignment()
{
	static SIZE_T OSPageSize = FPlatformMemory::GetConstants().PageSize;
	return OSPageSize;
}

FAndroidPlatformMemory::FPlatformVirtualMemoryBlock FAndroidPlatformMemory::FPlatformVirtualMemoryBlock::AllocateVirtual(size_t InSize, size_t InAlignment)
{
	FPlatformVirtualMemoryBlock Result;
	InSize = Align(InSize, GetVirtualSizeAlignment());
	Result.VMSizeDivVirtualSizeAlignment = InSize / GetVirtualSizeAlignment();

	size_t Alignment = FMath::Max(InAlignment, GetVirtualSizeAlignment());
	check(Alignment <= GetVirtualSizeAlignment());

	Result.Ptr = mmap(nullptr, Result.GetActualSize(), PROT_READ | PROT_WRITE, MAP_PRIVATE | MAP_ANON | MAP_NORESERVE, -1, 0);
	if (!LIKELY(Result.Ptr != MAP_FAILED))
	{
		FPlatformMemory::OnOutOfMemory(Result.GetActualSize(), InAlignment);
	}
	check(Result.Ptr && IsAligned(Result.Ptr, Alignment));
	SetMmapName(Result.Ptr, Result.GetActualSize(), "UE");
	return Result;
}



void FAndroidPlatformMemory::FPlatformVirtualMemoryBlock::FreeVirtual()
{
	if (Ptr)
	{
		check(VMSizeDivVirtualSizeAlignment > 0);
		if (munmap(Ptr, GetActualSize()) != 0)
		{
			// we can ran out of VMAs here
			FPlatformMemory::OnOutOfMemory(GetActualSize(), 0);
			// unreachable
		}
		Ptr = nullptr;
		VMSizeDivVirtualSizeAlignment = 0;
	}
}

void FAndroidPlatformMemory::FPlatformVirtualMemoryBlock::Commit(size_t InOffset, size_t InSize)
{
	check(IsAligned(InOffset, GetCommitAlignment()) && IsAligned(InSize, GetCommitAlignment()));
	check(InOffset >= 0 && InSize >= 0 && InOffset + InSize <= GetActualSize() && Ptr);
}

void FAndroidPlatformMemory::FPlatformVirtualMemoryBlock::Decommit(size_t InOffset, size_t InSize)
{
	check(IsAligned(InOffset, GetCommitAlignment()) && IsAligned(InSize, GetCommitAlignment()));
	check(InOffset >= 0 && InSize >= 0 && InOffset + InSize <= GetActualSize() && Ptr);
	if (madvise(((uint8*)Ptr) + InOffset, InSize, MADV_DONTNEED) != 0)
	{
		// we can ran out of VMAs here too!
		FPlatformMemory::OnOutOfMemory(InSize, 0);
	}
}


/**
* LLM uses these low level functions (LLMAlloc and LLMFree) to allocate memory. It grabs
* the function pointers by calling FPlatformMemory::GetLLMAllocFunctions. If these functions
* are not implemented GetLLMAllocFunctions should return false and LLM will be disabled.
*/

#if ENABLE_LOW_LEVEL_MEM_TRACKER

int64 LLMMallocTotal = 0;

void* LLMAlloc(size_t Size)
{
	void* Ptr = mmap(nullptr, Size, PROT_READ | PROT_WRITE, MAP_PRIVATE | MAP_ANON, -1, 0);
	if (Ptr == MAP_FAILED)
	{
		const int ErrNo = errno;
		UE_LOGF(LogHAL, Error, "mmap(Size=%llu) failed with errno = %d (%ls)", Size,
			ErrNo, StringCast< TCHAR >(strerror(ErrNo)).Get());
	}
	else
	{
		SetMmapName(Ptr, Size, "UE");

		LLMMallocTotal += Size;
	}

	return Ptr;
}

void LLMFree(void* Addr, size_t Size)
{
	LLMMallocTotal -= Size;
	if (Addr != nullptr && munmap(Addr, Size) != 0)
	{
		const int ErrNo = errno;
		UE_LOGF(LogHAL, Fatal, "munmap(addr=%p, len=%llu) failed with errno = %d (%ls)", Addr, Size,
			ErrNo, StringCast< TCHAR >(strerror(ErrNo)).Get());
	}
}

#endif

bool FAndroidPlatformMemory::GetLLMAllocFunctions(void*(*&OutAllocFunction)(size_t), void(*&OutFreeFunction)(void*, size_t), int32& OutAlignment)
{
#if ENABLE_LOW_LEVEL_MEM_TRACKER
	OutAllocFunction = LLMAlloc;
	OutFreeFunction = LLMFree;
	OutAlignment = sysconf(_SC_PAGESIZE);
	return true;
#else
	return false;
#endif
}

#if !UE_BUILD_SHIPPING

static std::atomic<bool> GAndroidMemoryProbeTickRun = {};
static FTSTicker::FDelegateHandle GAndroidMemoryProbeTicker = {};
static UE::Tasks::TTask<void> GAndroidMemoryProbeTask = {};

static float GAndroidMemoryProbeInterval = 10.0f;
static FAutoConsoleVariableRef CVarAndroidMemoryProbeInterval(
	TEXT("android.MemoryProbeInterval"),
	GAndroidMemoryProbeInterval,
	TEXT("/proc/self/smaps polling interval in seconds, default 10 seconds."),
	ECVF_Default
);

void FAndroidPlatformMemory::AndroidMemoryProbeStart()
{
	if (GAndroidMemoryProbeInterval <= 0.0f)
	{
		return;
	}

	if (GAndroidMemoryProbeTicker.IsValid())
	{
		return;
	}

	GAndroidMemoryProbeTickRun = true;

	GAndroidMemoryProbeTicker = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda([](float)
	{
		if (!GAndroidMemoryProbeTask.IsValid() || GAndroidMemoryProbeTask.IsCompleted())
		{
			GAndroidMemoryProbeTask = UE::Tasks::Launch(
				TEXT("AndroidMemoryProbeTick"),
				AndroidMemoryProbeTick,
				LowLevelTasks::ETaskPriority::BackgroundLow
			);
		}
		return true;
	}), GAndroidMemoryProbeInterval);
}

void FAndroidPlatformMemory::AndroidMemoryProbeStop()
{
	if (!GAndroidMemoryProbeTicker.IsValid())
	{
		return;
	}

	GAndroidMemoryProbeTickRun = false;

	if (GAndroidMemoryProbeTask.IsValid() && !GAndroidMemoryProbeTask.IsCompleted() && GAndroidMemoryProbeTask.IsAwaitable())
	{
		GAndroidMemoryProbeTask.Wait();
	}

	GAndroidMemoryProbeTask = {};

	FTSTicker::RemoveTicker(GAndroidMemoryProbeTicker);
	GAndroidMemoryProbeTicker = {};
}

void FAndroidPlatformMemory::AndroidMemoryProbeTick()
{
	if (!GAndroidMemoryProbeTickRun)
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(AndroidMemoryProbeTick2);

	FILE* Smaps = fopen("/proc/self/smaps", "r");
	if (!Smaps)
	{
		return;
	}

	char LineBuffer[1024] = { 0 };

	bool bIsProgramSize = false;
	uint64 ProgramSize = 0;

	while (GAndroidMemoryProbeTickRun)
	{
		char* LinePtr = fgets(LineBuffer, UE_ARRAY_COUNT(LineBuffer), Smaps);
		if (!LinePtr)
		{
			break; // eof or an error
		}
		FAnsiStringView Line(LinePtr);

		if (Line.Contains("-"))
		{
			// parsing region entry
			// 7fffcccb4000-7fffcccd8000 rw-p 00000000 00:00 0                          [stack]
			bIsProgramSize =
				FAnsiStringView(Line).EndsWith("/libUnreal.so\n") ||     // our own .so in case if it's unpacked
				FAnsiStringView(Line).EndsWith("/base.apk\n") ||         // our own .so in case if it's stored with-in .apk
				FAnsiStringView(Line).EndsWith("[anon:.bss]\n") ||
				FAnsiStringView(Line).EndsWith("[anon:linker_alloc]\n");
		}
		else if (bIsProgramSize && Line.StartsWith("Rss:"))
		{
			ProgramSize += AndroidPlatformMemory::GetBytesFromStatusLine(LinePtr);
		}
	}

	fclose(Smaps);

#if ENABLE_LOW_LEVEL_MEM_TRACKER
	FLowLevelMemTracker::Get().SetProgramSize(ProgramSize);
#endif
	FPlatformMemory::ProgramSize = ProgramSize;
}

#else

void FAndroidPlatformMemory::AndroidMemoryProbeStart()
{
}

void FAndroidPlatformMemory::AndroidMemoryProbeStop()
{
}

void FAndroidPlatformMemory::AndroidMemoryProbeTick()
{
}

#endif

#if USING_HW_ADDRESS_SANITISER && PLATFORM_USED_NDK_VERSION_INTEGER >= 26

// Using libc++_static on NDK r26b with HWAsan enabled is not officially supported
// In practice it does work, but when used with C++ 17 dynamic linker fails to find "_ZnamSt11align_val_t", "_ZnwmSt11align_val_t", "_ZdlPvSt11align_val_t", "_ZdaPvSt11align_val_t" when loading "libUnreal.so".
// It shouldn't be used by anything on UE side, so providing a stub as a temporary fix.

extern void* operator new(std::size_t Size, std::align_val_t Alignment)
{
	void* Result;
	if (UNLIKELY(posix_memalign(&Result, (std::size_t)Alignment, Size) != 0))
	{
		Result = nullptr;
	}
	return Result;
}

extern void* operator new[](std::size_t Size, std::align_val_t Alignment)
{
	void* Result;
	if (UNLIKELY(posix_memalign(&Result, (std::size_t)Alignment, Size) != 0))
	{
		Result = nullptr;
	}
	return Result;
}

extern void operator delete(void* Ptr, std::align_val_t)
{
	free(Ptr);
}

extern void operator delete[](void* Ptr, std::align_val_t)
{
	free(Ptr);
}

#endif