// Copyright Epic Games, Inc. All Rights Reserved.

#include "Android/AndroidPlatformMisc.h"
#include "Android/AndroidJniGameActivity.h"
#include "HAL/PlatformStackWalk.h"
#include "Misc/FileHelper.h"
#include "Misc/App.h"
#include "Misc/PathViews.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreDelegates.h"
#include "Misc/FeedbackContext.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Misc/EngineVersion.h"
#include "HAL/IConsoleManager.h"
#include "Modules/ModuleManager.h"
#include "BuildSettings.h"
#include <string.h>
#include <dlfcn.h>
#include <sys/statfs.h>
#include <sys/syscall.h>
#include <sched.h>
#include <sys/socket.h>
#include <sys/un.h>

#include "Android/AndroidPlatformCrashContext.h"
#include "Android/AndroidStats.h"
#include "HAL/PlatformMallocCrash.h"
#include "Android/AndroidJavaMessageBox.h"
#include "GenericPlatform/GenericPlatformChunkInstall.h"

#include "Misc/Parse.h"
#include "Misc/ScopeExit.h"
#include "Internationalization/Regex.h"

#include <android/trace.h>
#include <android/log.h>
#if USE_ANDROID_INPUT
#include <android/keycodes.h>
#endif
#if USE_ANDROID_JNI
#include <android/asset_manager.h>
#include <cpu-features.h>
#include <android_native_app_glue.h>
#include "Templates/Function.h"
#endif

#include "Misc/CoreDelegates.h"
#include "Async/TaskGraphInterfaces.h"
#include "Containers/Ticker.h"

#include "FramePro/FrameProProfiler.h"
#include <sys/mman.h>
#include "Templates/AlignmentTemplates.h"
#include "GenericPlatform/GenericPlatformOutputDevices.h"

#include "Android/AndroidPlatformStackWalk.h"
#include "Android/AndroidSignals.h"
#include "AndroidScudoMemoryTrace.h"

#include "Misc/OutputDevice.h"
#include "Logging/LogMacros.h"
#include "Misc/OutputDeviceError.h"
#include "Async/EventCount.h"

#include "VulkanCommon.h"
#include "IVulkanDynamicRHI.h"
#include <vulkan/vulkan_android.h>
#include "Android/AndroidPlatformProperties.h"
#include "ProfilingDebugging/CsvProfiler.h"

#include "AndroidVKQuality.h"

#if USE_ANDROID_JNI
extern AAssetManager * AndroidThunkCpp_GetAssetManager();
extern int32 GAndroidPackageVersion;
#else
#define GAndroidPackageVersion	1
#endif

#ifndef FORCE_ANDROID_LOG_TO_INFO_LEVEL
	#define FORCE_ANDROID_LOG_TO_INFO_LEVEL 0
#endif

// anything using LocalPrint (LowLevelOutputDebugString and therefore more generally, UE_LOG etc) will be sent to logcat via this log level.
// Note that <= ANDROID_LOG_DEBUG maybe filtered out by VS/AGDE.
android_LogPriority AndroidDefaultLogPriority()
{
	static const bool bPromoteLogLevel = !UE_BUILD_SHIPPING && (FORCE_ANDROID_LOG_TO_INFO_LEVEL || FPlatformMisc::IsDebuggerPresent());
	return bPromoteLogLevel ? ANDROID_LOG_INFO : ANDROID_LOG_DEBUG;
}

static int32 GAndroidTraceMarkersEnabled = 0;
static FAutoConsoleVariableRef CAndroidTraceMarkersEnabled(
	TEXT("android.tracemarkers"),
	GAndroidTraceMarkersEnabled,
	TEXT("Enable outputting named events to Android trace marker file.\n"),
	ECVF_Default
);

static int32 GAndroidLowPowerBatteryThreshold = 15;
static FAutoConsoleVariableRef CAndroidLowPowerBatteryThreshold(
	TEXT("android.LowPowerBatteryThreshold"),
	GAndroidLowPowerBatteryThreshold,
	TEXT("The battery level below which the device is considered in a low power state."),
	ECVF_Default
);

static TAutoConsoleVariable<int32> CVarMaliMidgardIndexingBug(
	TEXT("r.Android.MaliMidgardIndexingBug"),
	0,
	TEXT("For an indexed instance draw, the OpenGL ES driver does not handle attributes correctly. This issue only happens on Mali T8xx GPU when the difference between two adjacent index values are larger than 16.\n")
	TEXT("  0 = off\n")
	TEXT("  1 = on."),
	ECVF_ReadOnly
);

static TAutoConsoleVariable<FString> CVarAndroidCPUThermalSensorFilePath(
	TEXT("android.CPUThermalSensorFilePath"),
	"",
	TEXT("Overrides CPU Thermal sensor file path")
);

static float GAndroidMemoryStateChangeThreshold = 0.1f;
static FAutoConsoleVariableRef CAndroidMemoryStateChangeThreshold(
	TEXT("android.AndroidMemoryStateChangeThreshold"),
	GAndroidMemoryStateChangeThreshold,
	TEXT("The memory state change threshold after which memory state is reported to memory warning callback"),
	ECVF_Default
);

static bool GAndroidBroadcastIntentData = false;
static FAutoConsoleVariableRef CVarAndroidBroadcastIntentData(
	TEXT("android.BroadcastIntentData"),
	GAndroidBroadcastIntentData,
	TEXT("Whether to broadcast intent data, retry every frame if delegate is not bound"),
	ECVF_Default
);

#if STATS || ENABLE_STATNAMEDEVENTS
int32 FAndroidMisc::TraceMarkerFileDescriptor = -1;

#endif

// run time compatibility information
FString FAndroidMisc::AndroidVersion; // version of android we are running eg "4.0.4"
int32 FAndroidMisc::AndroidMajorVersion = 0; // integer major version of Android we are running, eg 10
int32 FAndroidMisc::TargetSDKVersion = 0; // Target SDK version, eg 29.
FString FAndroidMisc::DeviceMake; // make of the device we are running on eg. "samsung"
FString FAndroidMisc::DeviceModel; // model of the device we are running on eg "SAMSUNG-SGH-I437"
FString FAndroidMisc::DeviceBuildNumber; // platform image build number of device "R16NW.G960NKSU1ARD6"
FString FAndroidMisc::OSLanguage; // language code the device is set to eg "deu"
FString FAndroidMisc::ProductName; // product/marketing name of the device, if available.

// Build/API level we are running.
int32 FAndroidMisc::AndroidBuildVersion = 0;

// Whether or not the system handles the volume buttons (event will still be generated either way)
bool FAndroidMisc::VolumeButtonsHandledBySystem = true;

// Whether an app restart is needed to free driver allocated memory after precompiling PSOs
bool FAndroidMisc::bNeedsRestartAfterPSOPrecompile = false;

// Key/Value pair variables from the optional configuration.txt
TMap<FString, FString> FAndroidMisc::ConfigRulesVariables;

static FCriticalSection AndroidThreadNamesLock;
static TMap<uint32, const char*, TFixedSetAllocator<16>> AndroidThreadNames;

EDeviceScreenOrientation FAndroidMisc::DeviceOrientation = EDeviceScreenOrientation::Unknown;

extern void AndroidThunkCpp_ForceQuit();

extern void AndroidThunkCpp_SetOrientation(int32 Value);

#if USE_ANDROID_JNI
extern bool AndroidThunkCpp_HasSharedPreference(const FString& Group, const FString& Key);
extern void AndroidThunkCpp_SetSharedPreferenceInt(const FString& Group, const FString& Key, int32 Value);
extern int32 AndroidThunkCpp_GetSharedPreferenceInt(const FString& Group, const FString& Key, int32 DefaultValue);
extern void AndroidThunkCpp_SetSharedPreferenceString(const FString& Group, const FString& Key, const FString& Value);
extern bool AndroidThunkCpp_GetSharedPreferenceStringTypeSafe(const FString& Group, const FString& Key, FString& OutValue);
extern void AndroidThunkCpp_DeleteSharedPreference(const FString& Group, const FString& Key);
extern void AndroidThunkCpp_DeleteSharedPreferenceGroup(const FString& Group);
extern FString AndroidThunkCpp_GetIntentDataAsString();
#endif

// From AndroidFile.cpp
extern FString GFontPathBase;

static char AndroidCpuThermalSensorFileBuf[256] = "";

#if !UE_BUILD_SHIPPING
bool GDebugConsoleOpen; // has to be non-extern so LaunchAndroid.cpp can link with it
#endif

static void OverrideCpuThermalSensorFileFromCVar(IConsoleVariable* Var)
{
	FString Override = CVarAndroidCPUThermalSensorFilePath.GetValueOnAnyThread();
	const int32 Len = Override.Len();
	if (Len == 0)
	{
		return;
	}

	if (Len < UE_ARRAY_COUNT(AndroidCpuThermalSensorFileBuf))
	{
		FCStringAnsi::Strcpy(AndroidCpuThermalSensorFileBuf, TCHAR_TO_ANSI(*Override));
		UE_LOGF(LogAndroid, Display, "Thermal sensor's filepath was set to `%ls`", *Override);
		return;
	}

	UE_LOGF(LogAndroid, Display, "Thermal sensor's filepath is too long, max path is `%zu`", UE_ARRAY_COUNT(AndroidCpuThermalSensorFileBuf));
}

static void InitCpuThermalSensor()
{
	OverrideCpuThermalSensorFileFromCVar(nullptr);
	CVarAndroidCPUThermalSensorFilePath->SetOnChangedCallback(FConsoleVariableDelegate::CreateStatic(&OverrideCpuThermalSensorFileFromCVar));

	uint32 Counter = 0;
	const uint32 INVALID_INDEX = -1;
	uint32 CPUSensorIndex = INVALID_INDEX;
	while (true)
	{
		char Buf[256] = "";
		sprintf(Buf, "/sys/devices/virtual/thermal/thermal_zone%u/type", Counter);
		if (FILE* File = fopen(Buf, "r"))
		{
			fgets(Buf, UE_ARRAY_COUNT(Buf), File);
			fclose(File);
			char* Ptr = Buf;
			while (!iscntrl(*Ptr))		// it appears that zone type string ends up with \n symbol
			{
				++Ptr;
			}
			*Ptr = 0;

			if (strstr(Buf, "cpu-") && CPUSensorIndex == INVALID_INDEX)
			{
				CPUSensorIndex = Counter;
				FCStringAnsi::Sprintf(AndroidCpuThermalSensorFileBuf, "/sys/devices/virtual/thermal/thermal_zone%u/temp", Counter);
			}

			UE_LOGF(LogAndroid, Display, "Detected thermal sensor `%ls` at /sys/devices/virtual/thermal/thermal_zone%u/temp", ANSI_TO_TCHAR(Buf), Counter);
			++Counter;
		}
		else
		{
			break;
		}
	}

	TArray<FString> SensorLocations;
	GConfig->GetArray(TEXT("ThermalSensors"), TEXT("SensorLocations"), SensorLocations, GEngineIni);

	for (uint32 i = 0; i < SensorLocations.Num(); ++i)
	{
		auto ConvertedStr = StringCast<ANSICHAR>(*SensorLocations[i]);
		const char* SensorFilePath = ConvertedStr.Get();
		if (FILE* File = fopen(SensorFilePath, "r"))
		{
			FCStringAnsi::Strcpy(AndroidCpuThermalSensorFileBuf, SensorFilePath);
			UE_LOGF(LogAndroid, Display, "Selecting thermal sensor located at `%ls`", *SensorLocations[i]);
			fclose(File);
			return;
		}
	}

	if (CPUSensorIndex != INVALID_INDEX)
	{
		UE_LOGF(LogAndroid, Display, "Selecting thermal sensor located at `%ls`", ANSI_TO_TCHAR(AndroidCpuThermalSensorFileBuf));
	}
	else
	{
		UE_LOGF(LogAndroid, Display, "No CPU thermal sensor was detected. To manually override the sensor path set android.CPUThermalSensorFilePath CVar.");
	}
}

void FAndroidMisc::RequestExit( bool Force, const TCHAR* CallSite)
{

#if PLATFORM_COMPILER_OPTIMIZATION_PG_PROFILING
	// Write the PGO profiling file on a clean shutdown.
	extern void PGO_WriteFile();
	if (!GIsCriticalError)
	{
		PGO_WriteFile();
		// exit now to avoid a possible second PGO write when AndroidMain exits.
		Force = true;
	}
#endif

	UE_LOGF(LogAndroid, Log, "FAndroidMisc::RequestExit(%i, %ls)", Force,
		CallSite ? CallSite : TEXT("<NoCallSiteInfo>"));
	if (GLog)
	{
		GLog->Flush();
	}

	if (Force)
	{
#if USE_ANDROID_JNI
		AndroidThunkCpp_ForceQuit();
#else
		// for Android we should not exit with 1 when forcing exit
		exit(0);
#endif
	}
	else
	{
		RequestEngineExit(TEXT("Android RequestExit"));
	}
}

extern void AndroidThunkCpp_RestartApplication(const FString& IntentString);

bool FAndroidMisc::RestartApplication()
{
#if USE_ANDROID_JNI
	AndroidThunkCpp_RestartApplication(TEXT(""));
	return true;
#else
	return FGenericPlatformMisc::RestartApplication();
#endif
}

void FAndroidMisc::LocalPrint(const TCHAR *Message)
{
	// Builds for distribution should not have logging in them:
	// http://developer.android.com/tools/publishing/preparing.html#publishing-configure
#if !UE_BUILD_SHIPPING || ENABLE_PGO_PROFILE
	const int MAX_LOG_LENGTH = 4096;
	// not static since may be called by different threads
	wchar_t MessageBuffer[MAX_LOG_LENGTH];

	const TCHAR* SourcePtr = Message;
	while (*SourcePtr)
	{
		wchar_t* WritePtr = MessageBuffer;
		int32 RemainingSpace = MAX_LOG_LENGTH;
		while (*SourcePtr && --RemainingSpace > 0)
		{
			if (*SourcePtr == TEXT('\r'))
			{
				// If next character is newline, skip it
				if (*(++SourcePtr) == TEXT('\n'))
					++SourcePtr;
				break;
			}
			else if (*SourcePtr == TEXT('\n'))
			{
				++SourcePtr;
				break;
			}
			else {
				*WritePtr++ = static_cast<wchar_t>(*SourcePtr++);
			}
		}
		*WritePtr = '\0';
		__android_log_print(AndroidDefaultLogPriority(), "UE", "%ls", MessageBuffer);
	}
#endif
}

// Test for device vulkan support.
static void EstablishVulkanDeviceSupport();

namespace FAndroidAppEntry
{
	extern void PlatformInit();
}

void FAndroidMisc::PlatformPreInit()
{
	FAndroidCrashContext::Initialize();
	FGenericPlatformMisc::PlatformPreInit();
	EstablishVulkanDeviceSupport();
	FAndroidAppEntry::PlatformInit();
#if USE_ANDROID_JNI
	// Handle launch with intent
	HandleNewIntentUri(AndroidThunkCpp_GetIntentDataAsString());
#endif
}

static volatile bool HeadPhonesArePluggedIn = false;

static FAndroidMisc::FBatteryState CurrentBatteryState;

static FCriticalSection ReceiversLock;
static struct
{
	int		Volume;
	double	TimeOfChange;
} CurrentVolume;

#if USE_ANDROID_JNI && !USE_ANDROID_STANDALONE
namespace UE::Jni
{
	struct FHeadsetReceiver: Java::Lang::FObject
	{
		static constexpr FAnsiStringView ClassName = "com/epicgames/unreal/HeadsetReceiver";
		
		static void JNICALL stateChanged(JNIEnv * jni, jclass clazz, jint state)
		{
			FPlatformMisc::LowLevelOutputDebugStringf(TEXT("nativeHeadsetEvent(%i)"), state);
			HeadPhonesArePluggedIn = (state == 1);
		}
	};

	struct FVolumeReceiver: Java::Lang::FObject
	{
		static constexpr FAnsiStringView ClassName = "com/epicgames/unreal/VolumeReceiver";
		
		static void JNICALL volumeChanged(JNIEnv * jni, jclass clazz, jint volume)
		{
			FPlatformMisc::LowLevelOutputDebugStringf(TEXT("nativeVolumeEvent(%i)"), volume);
			ReceiversLock.Lock();
			CurrentVolume.Volume = volume;
			CurrentVolume.TimeOfChange = FApp::GetCurrentTime();
			ReceiversLock.Unlock();
		}
	};

	struct FBatteryReceiver: Java::Lang::FObject
	{
		static constexpr FAnsiStringView ClassName = "com/epicgames/unreal/BatteryReceiver";
		
		static void JNICALL dispatchEvent(JNIEnv * jni, jclass clazz, jint status, jint level, jint temperature)
		{
			FPlatformMisc::LowLevelOutputDebugStringf(TEXT("nativeBatteryEvent(stat = %i, lvl = %i %, temp = %3.2f \u00B0C)"), status, level, float(temperature)/10.f);

			ReceiversLock.Lock();
			const bool bWasInLowPowerMode = CurrentBatteryState.Level <= GAndroidLowPowerBatteryThreshold;

			FAndroidMisc::FBatteryState state;
			state.State = (FAndroidMisc::EBatteryState)status;
			state.Level = level;
			state.Temperature = float(temperature)/10.f;
			CurrentBatteryState = state;

			const bool bIsInLowPowerMode = CurrentBatteryState.Level <= GAndroidLowPowerBatteryThreshold;
			ReceiversLock.Unlock();

			// When we cross the low power battery level threshold, inform the active application
			if (bIsInLowPowerMode != bWasInLowPowerMode)
			{
				FGraphEventRef LowPowerTask = FFunctionGraphTask::CreateAndDispatchWhenReady([=]()
				{
					UE_LOGF(LogAndroid, Display, "Low Power Mode Changed: %d", bIsInLowPowerMode);
					FCoreDelegates::OnLowPowerMode.Broadcast(bIsInLowPowerMode);
				}, TStatId(), NULL, ENamedThreads::GameThread);	
			}
		}
	};
}
#endif

#if USE_ANDROID_JNI && !USE_ANDROID_STANDALONE

// Manage Java side OS event receivers.
static struct
{
	const char*		ClazzName;
	JNINativeMethod	Jnim;
	jclass			Clazz;
	jmethodID		StartReceiver;
	jmethodID		StopReceiver;
} JavaEventReceivers[] =
{
	{ UE::Jni::FVolumeReceiver::ClassName.GetData(),{ "volumeChanged", "(I)V",  (void*)UE::Jni::FVolumeReceiver::volumeChanged } },
	{ UE::Jni::FBatteryReceiver::ClassName.GetData(),{ "dispatchEvent", "(III)V",(void*)UE::Jni::FBatteryReceiver::dispatchEvent } },
	{ UE::Jni::FHeadsetReceiver::ClassName.GetData(),{ "stateChanged",  "(I)V",  (void*)UE::Jni::FHeadsetReceiver::stateChanged } },
};

void InitializeJavaEventReceivers()
{
	UE_LOGF(LogAndroid, Log, "InitializeJavaEventReceivers");

	// Register natives to receive Volume, Battery, Headphones events
	JNIEnv* JEnv = AndroidJavaEnv::GetJavaEnv();
	if (nullptr != JEnv)
	{
		auto CheckJNIExceptions = [&JEnv]()
		{
			if (JEnv->ExceptionCheck())
			{
				JEnv->ExceptionDescribe();
				JEnv->ExceptionClear();
			}
		};
		auto GetStaticMethod = [&JEnv, &CheckJNIExceptions](const char* MethodName, jclass Clazz, const char* ClazzName)
		{
			jmethodID Method = JEnv->GetStaticMethodID(Clazz, MethodName, "(Landroid/app/Activity;)V");
			if (Method == 0)
			{
				UE_LOGF(LogAndroid, Error, "Can't find method %ls of class %ls", ANSI_TO_TCHAR(MethodName), ANSI_TO_TCHAR(ClazzName));
			}
			CheckJNIExceptions();
			return Method;
		};

		for (auto& JavaEventReceiver : JavaEventReceivers)
		{
			JavaEventReceiver.Clazz = AndroidJavaEnv::FindJavaClassGlobalRef(JavaEventReceiver.ClazzName);
			if (JavaEventReceiver.Clazz == nullptr)
			{
				UE_LOGF(LogAndroid, Error, "Can't find class for %ls", ANSI_TO_TCHAR(JavaEventReceiver.ClazzName));
				continue;
			}
			if (JNI_OK != JEnv->RegisterNatives(JavaEventReceiver.Clazz, &JavaEventReceiver.Jnim, 1))
			{
				UE_LOGF(LogAndroid, Error, "RegisterNatives failed for %ls on %ls", ANSI_TO_TCHAR(JavaEventReceiver.ClazzName), ANSI_TO_TCHAR(JavaEventReceiver.Jnim.name));
				CheckJNIExceptions();
			}
			JavaEventReceiver.StartReceiver = GetStaticMethod("startReceiver", JavaEventReceiver.Clazz, JavaEventReceiver.ClazzName);
			JavaEventReceiver.StopReceiver = GetStaticMethod("stopReceiver", JavaEventReceiver.Clazz, JavaEventReceiver.ClazzName);
		}
	}
	else
	{
		UE_LOGF(LogAndroid, Warning, "Failed to initialize java event receivers. JNIEnv is not valid.");
	}
}

void EnableJavaEventReceivers(bool bEnableReceivers)
{
	JNIEnv* JEnv = AndroidJavaEnv::GetJavaEnv();
	if (nullptr != JEnv)
	{
		extern struct android_app* GNativeAndroidApp;
		for (auto& JavaEventReceiver : JavaEventReceivers)
		{
			jmethodID methodId = bEnableReceivers ? JavaEventReceiver.StartReceiver : JavaEventReceiver.StopReceiver;
			if (methodId != 0)
			{
				JEnv->CallStaticVoidMethod(JavaEventReceiver.Clazz, methodId, GNativeAndroidApp->activity->clazz);
			}
		}
	}
}

#endif	//USE_ANDROID_JNI


static FDelegateHandle AndroidOnBackgroundBinding;
static FDelegateHandle AndroidOnForegroundBinding;

#if (STATS || ENABLE_STATNAMEDEVENTS)

static void StartTraceMarkers()
{
	if (FAndroidMisc::TraceMarkerFileDescriptor != -1)
	{
		UE_LOGF(LogAndroid, Warning, "Systrace event logging already open.");
		return;
	}

	// Setup trace file descriptor
	FAndroidMisc::TraceMarkerFileDescriptor = open("/sys/kernel/debug/tracing/trace_marker", O_WRONLY);
	if (FAndroidMisc::TraceMarkerFileDescriptor == -1)
	{
		UE_LOGF(LogAndroid, Warning, "Trace Marker failed to open; systrace support disabled");
	}
	else
	{
		UE_LOGF(LogAndroid, Display, "Started systrace events logging.");
	}
}

static void StopTraceMarkers()
{
	// Tear down trace file descriptor
	if (FAndroidMisc::TraceMarkerFileDescriptor != -1)
	{
		close(FAndroidMisc::TraceMarkerFileDescriptor);
		FAndroidMisc::TraceMarkerFileDescriptor = -1;
		UE_LOGF(LogAndroid, Display, "Stopped systrace events logging.");
	}
}

static void UpdateTraceMarkersEnable(IConsoleVariable* Var)
{
	if (!GAndroidTraceMarkersEnabled)
	{
		StopTraceMarkers();
	}
	else
	{
		StartTraceMarkers();
	}
}
#endif

void FAndroidMisc::PlatformInit()
{
	// Increase the maximum number of simultaneously open files
	// Display Timer resolution.
	// Get swap file info
	// Display memory info
	// Setup user specified thread affinity if any
	extern void AndroidSetupDefaultThreadAffinity();
	AndroidSetupDefaultThreadAffinity();

#if USE_ANDROID_JNI && !USE_ANDROID_STANDALONE
	InitializeJavaEventReceivers();
	AndroidOnBackgroundBinding = FCoreDelegates::ApplicationWillEnterBackgroundDelegate.AddStatic(EnableJavaEventReceivers, false);
	AndroidOnForegroundBinding = FCoreDelegates::ApplicationHasEnteredForegroundDelegate.AddStatic(EnableJavaEventReceivers, true);

	extern void AndroidThunkJava_AddNetworkListener();
	AndroidThunkJava_AddNetworkListener();

	// Set initial network connection status
	FPlatformMisc::SetNetworkConnectionStatus(FPlatformMisc::NetworkConnectionTypeToStatus(GetNetworkConnectionType()));
#endif

	InitCpuThermalSensor();

	AndroidScudoMemoryTrace::Init();

	FAndroidPlatformMemory::AndroidMemoryProbeStart();

	CSV_METADATA( TEXT("Android_BuildNumber"), *GetDeviceBuildNumber() );
	CSV_METADATA( TEXT("Android_DeviceMake"),  *GetDeviceMake() );
	CSV_METADATA( TEXT("Android_DeviceModel"), *GetDeviceModel() );
	CSV_METADATA( TEXT("Android_Version"), *GetAndroidVersion() );
}

extern void AndroidThunkCpp_DismissSplashScreen();

void FAndroidMisc::PlatformTearDown()
{
	FAndroidPlatformMemory::AndroidMemoryProbeStop();

#if (STATS || ENABLE_STATNAMEDEVENTS)
	StopTraceMarkers();
#endif

	auto RemoveBinding = [](TMulticastDelegate<void()>& ApplicationLifetimeDelegate, FDelegateHandle& DelegateBinding)
	{
		if (DelegateBinding.IsValid())
		{
			ApplicationLifetimeDelegate.Remove(DelegateBinding);
			DelegateBinding.Reset();
		}
	};

	RemoveBinding(FCoreDelegates::ApplicationWillEnterBackgroundDelegate, AndroidOnBackgroundBinding);
	RemoveBinding(FCoreDelegates::ApplicationHasEnteredForegroundDelegate, AndroidOnForegroundBinding);
}

void FAndroidMisc::UpdateDeviceOrientation()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FAndroidMisc_UpdateDeviceOrientation);
#if USE_ANDROID_JNI && !USE_ANDROID_STANDALONE
	JNIEnv* JEnv = AndroidJavaEnv::GetJavaEnv();
	if (JEnv)
	{
		static jmethodID getOrientationMethod = 0;

		if (getOrientationMethod == 0)
		{
			jclass MainClass = AndroidJavaEnv::FindJavaClassGlobalRef("com/epicgames/unreal/GameActivity");
			if (MainClass != nullptr)
			{
				getOrientationMethod = JEnv->GetMethodID(MainClass, "AndroidThunkJava_GetDeviceOrientation", "()I");
				JEnv->DeleteGlobalRef(MainClass);
			}
		}

		if (getOrientationMethod != 0)
		{
			DeviceOrientation = (EDeviceScreenOrientation)JEnv->CallIntMethod(AndroidJavaEnv::GetGameActivityThis(), getOrientationMethod);
		}
	}
#endif
}

void FAndroidMisc::PlatformHandleSplashScreen(bool ShowSplashScreen)
{
#if USE_ANDROID_JNI && !USE_ANDROID_STANDALONE
	if (!ShowSplashScreen)
	{
		AndroidThunkCpp_DismissSplashScreen();
	}
	// Update the device orientation in case the game thread is blocked
	FAndroidMisc::UpdateDeviceOrientation();
#endif
}

FString FAndroidMisc::GetEnvironmentVariable(const TCHAR* VariableName)
{
	// @todo Android : get environment variable.
	return FString();
}

const TCHAR* FAndroidMisc::GetSystemErrorMessage(TCHAR* OutBuffer, int32 BufferCount, int32 Error)
{
	check(OutBuffer && BufferCount);
	*OutBuffer = TEXT('\0');
	if (Error == 0)
	{
		Error = errno;
	}
	char ErrorBuffer[1024];
	if (strerror_r(Error, ErrorBuffer, 1024) == 0)
	{
		FCString::Strncpy(OutBuffer, UTF8_TO_TCHAR((const ANSICHAR*)ErrorBuffer), BufferCount);
	}
	else
	{
		*OutBuffer = TEXT('\0');
	}
	return OutBuffer;
}

uint32 FAndroidMisc::GetLastError()
{
	return (uint32)errno;
}

EAppReturnType::Type FAndroidMisc::MessageBoxExt( EAppMsgType::Type MsgType, const TCHAR* Text, const TCHAR* Caption )
{
#if USE_ANDROID_JNI
	FJavaAndroidMessageBox MessageBox;
	MessageBox.SetText(Text);
	MessageBox.SetCaption(Caption);
	EAppReturnType::Type * ResultValues = nullptr;
	static EAppReturnType::Type ResultsOk[] = {
		EAppReturnType::Ok };
	static EAppReturnType::Type ResultsYesNo[] = {
		EAppReturnType::Yes, EAppReturnType::No };
	static EAppReturnType::Type ResultsOkCancel[] = {
		EAppReturnType::Ok, EAppReturnType::Cancel };
	static EAppReturnType::Type ResultsYesNoCancel[] = {
		EAppReturnType::Yes, EAppReturnType::No, EAppReturnType::Cancel };
	static EAppReturnType::Type ResultsCancelRetryContinue[] = {
		EAppReturnType::Cancel, EAppReturnType::Retry, EAppReturnType::Continue };
	static EAppReturnType::Type ResultsYesNoYesAllNoAll[] = {
		EAppReturnType::Yes, EAppReturnType::No, EAppReturnType::YesAll,
		EAppReturnType::NoAll };
	static EAppReturnType::Type ResultsYesNoYesAllNoAllCancel[] = {
		EAppReturnType::Yes, EAppReturnType::No, EAppReturnType::YesAll,
		EAppReturnType::NoAll, EAppReturnType::Cancel };
	static EAppReturnType::Type ResultsYesNoYesAll[] = {
		EAppReturnType::Yes, EAppReturnType::No, EAppReturnType::YesAll };

	// TODO: Should we localize button text?

	switch (MsgType)
	{
	case EAppMsgType::Ok:
		MessageBox.AddButton(TEXT("Ok"));
		ResultValues = ResultsOk;
		break;
	case EAppMsgType::YesNo:
		MessageBox.AddButton(TEXT("Yes"));
		MessageBox.AddButton(TEXT("No"));
		ResultValues = ResultsYesNo;
		break;
	case EAppMsgType::OkCancel:
		MessageBox.AddButton(TEXT("Ok"));
		MessageBox.AddButton(TEXT("Cancel"));
		ResultValues = ResultsOkCancel;
		break;
	case EAppMsgType::YesNoCancel:
		MessageBox.AddButton(TEXT("Yes"));
		MessageBox.AddButton(TEXT("No"));
		MessageBox.AddButton(TEXT("Cancel"));
		ResultValues = ResultsYesNoCancel;
		break;
	case EAppMsgType::CancelRetryContinue:
		MessageBox.AddButton(TEXT("Cancel"));
		MessageBox.AddButton(TEXT("Retry"));
		MessageBox.AddButton(TEXT("Continue"));
		ResultValues = ResultsCancelRetryContinue;
		break;
	case EAppMsgType::YesNoYesAllNoAll:
		MessageBox.AddButton(TEXT("Yes"));
		MessageBox.AddButton(TEXT("No"));
		MessageBox.AddButton(TEXT("Yes To All"));
		MessageBox.AddButton(TEXT("No To All"));
		ResultValues = ResultsYesNoYesAllNoAll;
		break;
	case EAppMsgType::YesNoYesAllNoAllCancel:
		MessageBox.AddButton(TEXT("Yes"));
		MessageBox.AddButton(TEXT("No"));
		MessageBox.AddButton(TEXT("Yes To All"));
		MessageBox.AddButton(TEXT("No To All"));
		MessageBox.AddButton(TEXT("Cancel"));
		ResultValues = ResultsYesNoYesAllNoAllCancel;
		break;
	case EAppMsgType::YesNoYesAll:
		MessageBox.AddButton(TEXT("Yes"));
		MessageBox.AddButton(TEXT("No"));
		MessageBox.AddButton(TEXT("Yes To All"));
		ResultValues = ResultsYesNoYesAll;
		break;
	default:
		check(0);
	}
	int32 Choice = MessageBox.Show();
	if (Choice >= 0 && nullptr != ResultValues)
	{
		return ResultValues[Choice];
	}
#endif

	// Failed to show dialog, or failed to get a response,
	// return default cancel response instead.
	return FGenericPlatformMisc::MessageBoxExt(MsgType, Text, Caption);
}

bool FAndroidMisc::HasPlatformFeature(const TCHAR* FeatureName)
{
	if (FCString::Stricmp(FeatureName, TEXT("Vulkan")) == 0)
	{
		return FAndroidMisc::ShouldUseVulkan();
	}

	return FGenericPlatformMisc::HasPlatformFeature(FeatureName);
}

static FString FixUpStoredValueSectionName(const FString& InSectionName)
{
	// We need to remove file separator character because SectionName is used as a file name by the OS
	if (InSectionName.Contains(TEXT("/")))
	{
		// Use extended replacement to avoid foo/bar and foo_bar mapping to each other
		return InSectionName.Replace(TEXT("/"), TEXT("___"));
	}
	else
	{
		return InSectionName;
	}
}

bool FAndroidMisc::SetStoredValue(const FString& InStoreId, const FString& InSectionName, const FString& InKeyName, const FString& InValue)
{
#if USE_ANDROID_JNI
	const FString FixedSectionValue = FixUpStoredValueSectionName(InSectionName);
	AndroidThunkCpp_SetSharedPreferenceString(FixedSectionValue, InKeyName, InValue);
	return true;
#else
	return FGenericPlatformMisc::SetStoredValue(InStoreId, InSectionName, InKeyName, InValue);
#endif
}

bool FAndroidMisc::GetStoredValue(const FString& InStoreId, const FString& InSectionName, const FString& InKeyName, FString& OutValue)
{
#if USE_ANDROID_JNI
	const FString FixedSectionValue = FixUpStoredValueSectionName(InSectionName);
	if (AndroidThunkCpp_GetSharedPreferenceStringTypeSafe(FixedSectionValue, InKeyName, OutValue))
	{
		return true;
	}
#endif

	// fallback to FGenericPlatformMisc::GetStoredValue
	return FGenericPlatformMisc::GetStoredValue(InStoreId, InSectionName, InKeyName, OutValue);
}

bool FAndroidMisc::DeleteStoredValue(const FString& InStoreId, const FString& InSectionName, const FString& InKeyName)
{
	bool bResult = false;
#if USE_ANDROID_JNI
	const FString FixedSectionValue = FixUpStoredValueSectionName(InSectionName);
	// delete doesn't have a return value, our best effort is to check for preference existence first
	bResult = AndroidThunkCpp_HasSharedPreference(FixedSectionValue, InKeyName);
	AndroidThunkCpp_DeleteSharedPreference(FixedSectionValue, InKeyName);
#endif
	// always delete in both places just in case
	bResult |= FGenericPlatformMisc::DeleteStoredValue(InStoreId, InSectionName, InKeyName);
	return bResult;
}

bool FAndroidMisc::DeleteStoredSection(const FString& InStoreId, const FString& InSectionName)
{
#if USE_ANDROID_JNI
	const FString FixedSectionValue = FixUpStoredValueSectionName(InSectionName);
	// can't easily check if section exists as Context.getSharedPreferences will create object if it doesn't exist
	AndroidThunkCpp_DeleteSharedPreferenceGroup(FixedSectionValue);
#endif
	// always delete in both places just in case
	return FGenericPlatformMisc::DeleteStoredSection(InStoreId, InSectionName);
}

FString FAndroidMisc::GetBillingConfigCountryCode()
{
#if USE_ANDROID_JNI
	extern FString AndroidThunkJava_IapGetBillingCountryCode();
	return AndroidThunkJava_IapGetBillingCountryCode(); 
#else
	return FString();
#endif
}

bool FAndroidMisc::GetIsAlternativeBillingOnlyAvailable()
{
#if USE_ANDROID_JNI
	extern bool AndroidThunkJava_IapGetIsAlternativeBillingOnlyAvailable();
	return AndroidThunkJava_IapGetIsAlternativeBillingOnlyAvailable();
#else
	return true;
#endif
}

bool FAndroidMisc::UseRenderThread()
{
	// if we in general don't want to use the render thread due to commandline, etc, then don't
	if (!FGenericPlatformMisc::UseRenderThread())
	{
		return false;
	}

	// Check for DisableThreadedRendering CVar from DeviceProfiles config
	// Any devices in the future that need to disable threaded rendering should be given a device profile and use this CVar
	const IConsoleVariable *const CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.AndroidDisableThreadedRendering"));
	if (CVar && CVar->GetInt() != 0)
	{
		return false;
	}

	// there is a crash with the nvidia tegra dual core processors namely the optimus 2x and xoom
	// when running multithreaded it can't handle multiple threads using opengl (bug)
	// tested with lg optimus 2x and motorola xoom
	// come back and revisit this later
	// https://code.google.com/p/android/issues/detail?id=32636
	if (FAndroidMisc::GetGPUFamily() == FString("NVIDIA Tegra") && FPlatformMisc::NumberOfCores() <= 2 && FAndroidMisc::GetGLVersion().StartsWith(TEXT("OpenGL ES 2.")))
	{
		return false;
	}

	// Vivante GC1000 with 2.x driver has issues with render thread
	if (FAndroidMisc::GetGPUFamily().StartsWith(TEXT("Vivante GC1000")) && FAndroidMisc::GetGLVersion().StartsWith(TEXT("OpenGL ES 2.")))
	{
		return false;
	}

	// there is an issue with presenting the buffer on kindle fire (1st gen) with multiple threads using opengl
	if (FAndroidMisc::GetDeviceModel() == FString("Kindle Fire"))
	{
		return false;
	}

	// there is an issue with swapbuffer ordering on startup on samsung s3 mini with multiple threads using opengl
	if (FAndroidMisc::GetDeviceModel() == FString("GT-I8190L"))
	{
		return false;
	}

	return true;
}

int32 FAndroidMisc::NumberOfCores()
{
#if USE_ANDROID_JNI
	int32 NumberOfCores = android_getCpuCount();
#else
	int32 NumberOfCores = 0;
#endif

	static int CalculatedNumberOfCores = 0;
	if (CalculatedNumberOfCores == 0)
	{
		pid_t ThreadId = gettid();
		cpu_set_t cpuset;
		CPU_ZERO(&cpuset);
		if (sched_getaffinity(ThreadId, sizeof(cpuset), &cpuset) != -1)
		{
			CalculatedNumberOfCores = CPU_COUNT(&cpuset);
		}

		UE_LOGF(LogTemp, Log, "%d cores and %d assignable cores", NumberOfCores, CalculatedNumberOfCores);
	}

	return !CalculatedNumberOfCores ? NumberOfCores : CalculatedNumberOfCores;
}

int32 FAndroidMisc::NumberOfCoresIncludingHyperthreads()
{
	return NumberOfCores();
}


static FAndroidMisc::FCPUState CurrentCPUState;

FAndroidMisc::FCPUState& FAndroidMisc::GetCPUState(){
	uint64_t UserTime, NiceTime, SystemTime, SoftIRQTime, IRQTime, IdleTime, WallTime, IOWaitTime;
	int32		Index = 0;
	ANSICHAR	Buffer[500];

	CurrentCPUState.CoreCount = FMath::Min( FAndroidMisc::NumberOfCores(), FAndroidMisc::FCPUState::MaxSupportedCores);
	FILE* FileHandle = fopen("/proc/stat", "r");
	if (FileHandle){
		CurrentCPUState.ActivatedCoreCount = 0;
		for (size_t n = 0; n < CurrentCPUState.CoreCount; n++) {
			CurrentCPUState.Status[n] = 0;
			CurrentCPUState.PreviousUsage[n] = CurrentCPUState.CurrentUsage[n];
		}

		while (fgets(Buffer, 100, FileHandle)) {
			sscanf(Buffer, "%5s %8lu %8lu %8lu %8lu %8lu %8lu %8lu", CurrentCPUState.Name,
				&UserTime, &NiceTime, &SystemTime, &IdleTime, &IOWaitTime, &IRQTime,
				&SoftIRQTime);

			if (0 == strncmp(CurrentCPUState.Name, "cpu", 3)) {
				Index = CurrentCPUState.Name[3] - '0';
				if (Index >= 0 && Index < CurrentCPUState.CoreCount)
				{
					if(CurrentCPUState.Name[5] != '\0')
					{
						Index = atol(&CurrentCPUState.Name[3]);
					}
					CurrentCPUState.CurrentUsage[Index].IdleTime = IdleTime;
					CurrentCPUState.CurrentUsage[Index].NiceTime = NiceTime;
					CurrentCPUState.CurrentUsage[Index].SystemTime = SystemTime;
					CurrentCPUState.CurrentUsage[Index].SoftIRQTime = SoftIRQTime;
					CurrentCPUState.CurrentUsage[Index].IRQTime = IRQTime;
					CurrentCPUState.CurrentUsage[Index].IOWaitTime = IOWaitTime;
					CurrentCPUState.CurrentUsage[Index].UserTime = UserTime;
					CurrentCPUState.CurrentUsage[Index].TotalTime = UserTime + NiceTime + SystemTime + SoftIRQTime + IRQTime + IdleTime + IOWaitTime;
					CurrentCPUState.Status[Index] = 1;
					CurrentCPUState.ActivatedCoreCount++;
				}
				if (Index == CurrentCPUState.CoreCount-1)
					break;
			}
		}
		fclose(FileHandle);

		CurrentCPUState.AverageUtilization = 0.0;
		for (size_t n = 0; n < CurrentCPUState.CoreCount; n++) {
			if (CurrentCPUState.CurrentUsage[n].TotalTime <= CurrentCPUState.PreviousUsage[n].TotalTime) {
				continue;
			}

			WallTime = CurrentCPUState.CurrentUsage[n].TotalTime - CurrentCPUState.PreviousUsage[n].TotalTime;
			IdleTime = CurrentCPUState.CurrentUsage[n].IdleTime - CurrentCPUState.PreviousUsage[n].IdleTime;

			if (!WallTime || WallTime <= IdleTime) {
				continue;
			}
			CurrentCPUState.Utilization[n] = ((double)WallTime - (double)IdleTime) * 100.0 / (double)WallTime;
			CurrentCPUState.AverageUtilization += CurrentCPUState.Utilization[n];
		}
		CurrentCPUState.AverageUtilization /= (double)CurrentCPUState.CoreCount;
	}else{
		FMemory::Memzero(CurrentCPUState);
	}
	return CurrentCPUState;
}



bool FAndroidMisc::SupportsLocalCaching()
{
	return true;

	/*if ( SupportsUTime() )
	{
		return true;
	}*/


}

static int SysGetRandomSupported = -1;

// http://man7.org/linux/man-pages/man2/getrandom.2.html
// getrandom() was introduced in version 3.17 of the Linux kernel
//   and glibc version 2.25.

// Check known platforms if SYS_getrandom isn't defined
#if !defined(SYS_getrandom)
	#if PLATFORM_CPU_X86_FAMILY
		#define SYS_getrandom 318
	#elif PLATFORM_CPU_ARM_FAMILY
		#define SYS_getrandom 278
	#endif
#endif // !defined(SYS_getrandom)

namespace
{
#if defined(SYS_getrandom)

#if !defined(GRND_NONBLOCK)
	#define GRND_NONBLOCK 0x0001
#endif

	int SysGetRandom(void *buf, size_t buflen)
	{
		if (SysGetRandomSupported < 0)
		{
			int Ret = syscall(SYS_getrandom, buf, buflen, GRND_NONBLOCK);

			// If -1 is returned with ENOSYS, kernel doesn't support getrandom
			SysGetRandomSupported = ((Ret == -1) && (errno == ENOSYS)) ? 0 : 1;
		}

		return SysGetRandomSupported ?
			syscall(SYS_getrandom, buf, buflen, GRND_NONBLOCK) : -1;
	}

#else

	int SysGetRandom(void *buf, size_t buflen)
	{
		return -1;
	}

#endif // !SYS_getrandom
}

/**
 * Try to use SYS_getrandom which would be the fastest, otherwise fall back to 
 * use /proc/sys/kernel/random/uuid to get GUID; do NOT use JNI since this may be called too early
 */
void  FAndroidMisc::CreateGuid(struct FGuid& Result)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FGenericPlatformMisc_CreateGuid);

	static bool bGetRandomFailed = false;
	static bool bProcUUIDFailed = false;

	if (!bGetRandomFailed)
	{
		int BytesRead = SysGetRandom(&Result, sizeof(Result));

		if (BytesRead == sizeof(Result))
		{
			// https://tools.ietf.org/html/rfc4122#section-4.4
			// https://en.wikipedia.org/wiki/Universally_unique_identifier
			//
			// The 4 bits of digit M indicate the UUID version, and the 1â€“3
			//   most significant bits of digit N indicate the UUID variant.
			// xxxxxxxx-xxxx-Mxxx-Nxxx-xxxxxxxxxxxx
			Result[1] = (Result[1] & 0xffff0fff) | 0x00004000; // version 4
			Result[2] = (Result[2] & 0x3fffffff) | 0x80000000; // variant 1
			return;
		}
		bGetRandomFailed = true;
	}

#define FROM_HEX(_a) ( (_a) <= '9' ? (_a) - '0' : (_a) <= 'F' ?  (_a) - 'A' + 10 : (_a) - 'a' + 10 )

	if (!bProcUUIDFailed)
	{
		int32 Handle = open("/proc/sys/kernel/random/uuid", O_RDONLY);
		if (Handle != -1)
		{
			char LineBuffer[36];
			int ReadBytes = read(Handle, LineBuffer, 36);
			close(Handle);
			if (ReadBytes == 36)
			{
				Result.A = FROM_HEX(LineBuffer[0]) << 28 | FROM_HEX(LineBuffer[1]) << 24 | FROM_HEX(LineBuffer[2]) << 20 | FROM_HEX(LineBuffer[3]) << 16 |
					FROM_HEX(LineBuffer[4]) << 12 | FROM_HEX(LineBuffer[5]) << 8 | FROM_HEX(LineBuffer[6]) << 4 | FROM_HEX(LineBuffer[7]);
				Result.B = FROM_HEX(LineBuffer[9]) << 28 | FROM_HEX(LineBuffer[10]) << 24 | FROM_HEX(LineBuffer[11]) << 20 | FROM_HEX(LineBuffer[12]) << 16 |
					FROM_HEX(LineBuffer[14]) << 12 | FROM_HEX(LineBuffer[15]) << 8 | FROM_HEX(LineBuffer[16]) << 4 | FROM_HEX(LineBuffer[17]);
				Result.C = FROM_HEX(LineBuffer[19]) << 28 | FROM_HEX(LineBuffer[20]) << 24 | FROM_HEX(LineBuffer[21]) << 20 | FROM_HEX(LineBuffer[22]) << 16 |
					FROM_HEX(LineBuffer[24]) << 12 | FROM_HEX(LineBuffer[25]) << 8 | FROM_HEX(LineBuffer[26]) << 4 | FROM_HEX(LineBuffer[27]);
				Result.D = FROM_HEX(LineBuffer[28]) << 28 | FROM_HEX(LineBuffer[29]) << 24 | FROM_HEX(LineBuffer[30]) << 20 | FROM_HEX(LineBuffer[31]) << 16 |
					FROM_HEX(LineBuffer[32]) << 12 | FROM_HEX(LineBuffer[33]) << 8 | FROM_HEX(LineBuffer[34]) << 4 | FROM_HEX(LineBuffer[35]);
				return;
			}
		}
		bProcUUIDFailed = true;
	}

#undef FROM_HEX

	// fall back to generic CreateGuid
	FGenericPlatformMisc::CreateGuid(Result);
}

/**
 * Good enough default crash reporter.
 */
void DefaultCrashHandler(const FAndroidCrashContext& Context)
{
	static int32 bHasEntered = 0;
	if (FPlatformAtomics::InterlockedCompareExchange(&bHasEntered, 1, 0) == 0)
	{
		const SIZE_T StackTraceSize = 65535;
		ANSICHAR StackTrace[StackTraceSize];
		StackTrace[0] = 0;

		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Starting StackWalk..."));

		// Walk the stack and dump it to the allocated memory.
		FPlatformStackWalk::StackWalkAndDump(StackTrace, StackTraceSize, 0, Context.Context);
		UE_LOGF(LogAndroid, Error, "\n%ls\n", ANSI_TO_TCHAR(StackTrace));

		if (GLog)
		{
			GLog->Panic();
		}

		if (GWarn)
		{
			GWarn->Flush();
		}
	}
}

/** Global pointer to crash handler */
void (* GCrashHandlerPointer)(const FGenericCrashContext& Context) = NULL;

static constexpr int32 TargetSignals[] =
{
	SIGQUIT, // SIGQUIT is a user-initiated "crash".
	SIGILL,
	SIGFPE,
	SIGBUS,
	SIGSEGV,
	SIGSYS,
	SIGABRT
};
static constexpr int32 NumTargetSignals = UE_ARRAY_COUNT(TargetSignals);

static const char* SignalToString(int32 Signal)
{
	switch (Signal)
	{
		case SIGQUIT:
			return "SIGQUIT";
		case SIGILL:
			return "SIGILL";
		case SIGFPE:
			return "SIGFPE";
		case SIGBUS:
			return "SIGBUS";
		case SIGSEGV:
			return "SIGSEGV";
		case SIGSYS:
			return "SIGSYS";
		case SIGABRT:
			return "SIGABRT";
		default:
			return FAndroidCrashContext::ItoANSI(Signal,16, 16);
	}
}

#if ANDROID_HAS_RTSIGNALS

float GAndroidSignalTimeOut = 20.0f;
static FAutoConsoleVariableRef CAndroidSignalTimeout(
	TEXT("android.SignalTimeout"),
	GAndroidSignalTimeOut,
	TEXT("Time in seconds to wait for the signal handler to complete before timing out and terminating the process."),
	ECVF_Default
);

template<typename Derived >
typename FSignalHandler< Derived >::FSignalParams FSignalHandler< Derived >::SignalParams;
template<typename Derived >
int32 FSignalHandler< Derived >::SignalThreadStatus = (int32)FSignalHandler< Derived >::ESignalThreadStatus::NotInitialized;
template<typename Derived >
uint32 FSignalHandler< Derived >::ForwardingThreadID = 0xffffffff;
template<typename Derived >
int32 FSignalHandler< Derived >::ForwardingSignalType = -1;
template<typename Derived >
struct sigaction FSignalHandler< Derived >::PreviousActionForForwardSignal;


class FThreadCallstackSignalHandler : FSignalHandler<FThreadCallstackSignalHandler>
{
	friend class FSignalHandler<FThreadCallstackSignalHandler>;
public:
	static void Init()
	{
		FSignalHandler<FThreadCallstackSignalHandler>::Init(THREADBACKTRACE_SIGNAL_FWD);
		HookTargetSignal();
	}

	static void Release()
	{
		RestorePreviousTargetSignalHandler();
		FSignalHandler<FThreadCallstackSignalHandler>::Release();
	}

private:
	static void OnTargetSignal(int Signal, siginfo* Info, void* Context)
	{
		while (FPlatformAtomics::InterlockedCompareExchange(&handling_signal, 1, 0) != 0)
		{
			FPlatformProcess::SleepNoStats(0.0f);
		}
		FSignalHandler<FThreadCallstackSignalHandler>::ForwardSignal(Signal, Info, Context);
		FPlatformAtomics::AtomicStore(&handling_signal, 0);
	}

	static void HandleTargetSignal(int Signal, siginfo* Info, void* Context, uint32 CrashingThreadId)
	{
		FPlatformStackWalk::HandleBackTraceSignal(Info, Context);
	}

	static void HookTargetSignal()
	{
		check(bSignalHooked == false);
		struct sigaction ActionForThread;
		FMemory::Memzero(ActionForThread);
		sigfillset(&ActionForThread.sa_mask);
		ActionForThread.sa_flags = SA_SIGINFO | SA_RESTART | SA_ONSTACK;
		ActionForThread.sa_sigaction = &OnTargetSignal;
		sigaction(THREAD_CALLSTACK_GENERATOR, &ActionForThread, &PreviousActionForThreadGenerator);
		bSignalHooked = true;
	}

	static void RestorePreviousTargetSignalHandler()
	{
		if (bSignalHooked)
		{
			bSignalHooked = false;
			sigaction(THREAD_CALLSTACK_GENERATOR, &PreviousActionForThreadGenerator, nullptr);
		}
	}

	static bool bSignalHooked;
	static volatile sig_atomic_t handling_signal;
	static struct sigaction PreviousActionForThreadGenerator;
};

const ANSICHAR* FAndroidMisc::CodeToString(int Signal, int si_code)
{
	switch (Signal)
	{
		case SIGILL:
		{
			switch (si_code)
			{
				// SIGILL
				case ILL_ILLOPC: return "ILL_ILLOPC";
				case ILL_ILLOPN: return "ILL_ILLOPN";
				case ILL_ILLADR: return "ILL_ILLADR";
				case ILL_ILLTRP: return "ILL_ILLTRP";
				case ILL_PRVOPC: return "ILL_PRVOPC";
				case ILL_PRVREG: return "ILL_PRVREG";
				case ILL_COPROC: return "ILL_COPROC";
				case ILL_BADSTK: return "ILL_BADSTK";
			}
		}
		break;
		case SIGFPE:
		{
			switch (si_code)
			{
				// SIGFPE
				case FPE_INTDIV: return "FPE_INTDIV";
				case FPE_INTOVF: return "FPE_INTOVF";
				case FPE_FLTDIV: return "FPE_FLTDIV";
				case FPE_FLTOVF: return "FPE_FLTOVF";
				case FPE_FLTUND: return "FPE_FLTUND";
				case FPE_FLTRES: return "FPE_FLTRES";
				case FPE_FLTINV: return "FPE_FLTINV";
				case FPE_FLTSUB: return "FPE_FLTSUB";
			}
		}
		break;
		case SIGBUS:
		{
			switch (si_code)
			{
				// SIGBUS
				case BUS_ADRALN: return "BUS_ADRALN";
				case BUS_ADRERR: return "BUS_ADRERR";
				case BUS_OBJERR: return "BUS_OBJERR";
			}
		}
		break;
		case SIGSEGV:
		{
			switch (si_code)
			{
				// SIGSEGV
				case SEGV_MAPERR: return "SEGV_MAPERR";
				case SEGV_ACCERR: return "SEGV_ACCERR";
			}
		}
		break;
	}
	return FAndroidCrashContext::ItoANSI(si_code, 10, 0);
}

FString FAndroidMisc::GetFatalSignalMessage(int Signal, siginfo* Info)
{
	const int MessageSize = 255;
	char AnsiMessage[MessageSize];
	FCStringAnsi::Strncpy(AnsiMessage, "Caught signal : ", MessageSize);
	FCStringAnsi::Strcat(AnsiMessage, SignalToString(Signal));
	FCStringAnsi::Strcat(AnsiMessage, " (");
	FCStringAnsi::Strcat(AnsiMessage, CodeToString(Signal, Info->si_code));
	FCStringAnsi::Strcat(AnsiMessage, ")");
	switch (Signal)
	{
		case SIGILL:
		case SIGFPE:
		case SIGSEGV:
		case SIGBUS:
		case SIGTRAP:
		{
			FCStringAnsi::Strcat(AnsiMessage, " fault address 0x");
			FCStringAnsi::Strcat(AnsiMessage, FAndroidCrashContext::ItoANSI((uintptr_t)Info->si_addr, 16, 16));
			break;
		}
	}

	return ANSI_TO_TCHAR(AnsiMessage);
}

// Making the signal handler available to track down issues with failing crash handler.
static void (*GFatalSignalHandlerOverrideFunc)(int Signal, struct siginfo* Info, void* Context, uint32 CrashingThreadId) = nullptr;
void FAndroidMisc::OverrideFatalSignalHandler(void (*FatalSignalHandlerOverrideFunc)(int Signal, struct siginfo* Info, void* Context, uint32 CrashingThreadId))
{
	GFatalSignalHandlerOverrideFunc = FatalSignalHandlerOverrideFunc;
}

volatile sig_atomic_t FThreadCallstackSignalHandler::handling_signal = 0;
bool FThreadCallstackSignalHandler::bSignalHooked = false;
struct sigaction FThreadCallstackSignalHandler::PreviousActionForThreadGenerator;

class FFatalSignalHandler : public FSignalHandler<FFatalSignalHandler>
{
	friend class FSignalHandler<FFatalSignalHandler>;
public:
	static void Init()
	{
		FSignalHandler<FFatalSignalHandler>::Init(FATAL_SIGNAL_FWD);
		HookTargetSignals();
	}

	static void Release()
	{
		RestorePreviousTargetSignalHandlers();
		FSignalHandler<FFatalSignalHandler>::Release();
	}

	static bool IsInFatalSignalHandler()
	{
		return FPlatformAtomics::AtomicRead(&handling_fatal_signal) > 0;
	}

protected:

	static void EnterFatalCrash()
	{
		// we are a fatal signal, we can only handle one at a time. So avoid allow multiple fatal signals going through
		if (FPlatformAtomics::InterlockedIncrement(&handling_fatal_signal) != 1)
		{
			FPlatformProcess::SleepNoStats(60.0f);
			// exit immediately, crash malloc can cause deadlocks when attempting to clean up static objects via exit().
			_exit(1);
		}
	}

	static void OnTargetSignal(int Signal, siginfo* Info, void* Context)
	{
		EnterFatalCrash();
		FSignalHandler<FFatalSignalHandler>::ForwardSignal(Signal, Info, Context);
		RestorePreviousTargetSignalHandlers();

		// re-raise the signal for the benefit of the previous handler.
		raise(Signal);
	}

	static void HandleTargetSignal(int Signal, siginfo* Info, void* Context, uint32 CrashingThreadId)
	{
		if (GFatalSignalHandlerOverrideFunc)
		{
			GFatalSignalHandlerOverrideFunc(Signal, Info, Context, CrashingThreadId);
		}
		else
		{
			// Switch to malloc crash.
			FPlatformMallocCrash::Get().SetAsGMalloc();

			FString Message = FAndroidMisc::GetFatalSignalMessage(Signal, Info);
			FAndroidCrashContext CrashContext(ECrashContextType::Crash, *Message);

			CrashContext.InitFromSignal(Signal, Info, Context, CrashingThreadId);
			CrashContext.CaptureCrashInfo();
			if (GCrashHandlerPointer)
			{
				GCrashHandlerPointer(CrashContext);
			}
			else
			{
				// call default one
				DefaultCrashHandler(CrashContext);
			}
		}
	}

	static void HookTargetSignals()
	{
		check(PreviousSignalHandlersValid == false);
		// hook our signals and record current set.
		struct sigaction Action;
		FMemory::Memzero(&Action, sizeof(struct sigaction));
		Action.sa_sigaction = &OnTargetSignal;
		// sigfillset will block all other signals whilst the signal handler is processing.
		sigfillset(&Action.sa_mask);
		Action.sa_flags = SA_SIGINFO | SA_RESTART | SA_ONSTACK;

		for (int32 i = 0; i < NumTargetSignals; ++i)
		{
			int result = sigaction(TargetSignals[i], &Action, &PrevActions[i]);
			UE_CLOGF(result != 0, LogAndroid, Error, "sigaction(%d) failed to set: %d, errno = %x ", i, result, errno);
		}
		PreviousSignalHandlersValid = true;
	}

	static void RestorePreviousTargetSignalHandlers()
	{
		if (PreviousSignalHandlersValid)
		{
			for (int32 i = 0; i < NumTargetSignals; ++i)
			{
				int result = sigaction(TargetSignals[i], &PrevActions[i], NULL);
				UE_CLOGF(result != 0, LogAndroid, Error, "sigaction(%d) failed to set prev action: %d, errno = %x ", i, result, errno);
			}
			PreviousSignalHandlersValid = false;
		}
	}

	static volatile sig_atomic_t handling_fatal_signal;
	static struct sigaction PrevActions[NumTargetSignals];
	static bool PreviousSignalHandlersValid;
};

volatile sig_atomic_t FFatalSignalHandler::handling_fatal_signal = 0;
struct sigaction FFatalSignalHandler::PrevActions[NumTargetSignals];
bool FFatalSignalHandler::PreviousSignalHandlersValid = false;
#endif// ANDROID_HAS_RTSIGNALS

static void SetDefaultSignalHandlers()
{
	struct sigaction Action;
	FMemory::Memzero(&Action, sizeof(struct sigaction));
	Action.sa_handler = SIG_DFL;
	sigemptyset(&Action.sa_mask);

	for (int32 i = 0; i < NumTargetSignals; ++i)
	{
		sigaction(TargetSignals[i], &Action, NULL);
	}
}

bool FAndroidMisc::IsInSignalHandler()
{
#if ANDROID_HAS_RTSIGNALS
	return FFatalSignalHandler::IsInFatalSignalHandler();
#else
	return false;
#endif
}

void FAndroidMisc::TriggerCrashHandler(ECrashContextType InType, const TCHAR* InErrorMessage, const TCHAR* OverrideCallstack)
{
	if (InType != ECrashContextType::Crash)
	{
		// we dont flush logs during a fatal signal, malloccrash can cause us to deadlock.
		if (GLog)
		{
			GLog->Panic();
		}
		if (GWarn)
		{
			GWarn->Flush();
		}
		if (GError)
		{
			GError->Flush();
		}
	}

	FAndroidCrashContext CrashContext(InType, InErrorMessage);

	if (OverrideCallstack)
	{
		CrashContext.SetOverrideCallstack(OverrideCallstack);
	}
	else
	{
		CrashContext.CaptureCrashInfo();
	}

	if (GCrashHandlerPointer)
	{
		GCrashHandlerPointer(CrashContext);
	}
	else
	{
		// call default one
		DefaultCrashHandler(CrashContext);
	}
}

void FAndroidMisc::SetCrashHandler(void(*CrashHandler)(const FGenericCrashContext& Context))
{
#if ANDROID_HAS_RTSIGNALS
	UE_LOGF(LogAndroid, Log, "Setting Crash Handler = %p", CrashHandler);

	GCrashHandlerPointer = CrashHandler;

	FFatalSignalHandler::Release();
	FThreadCallstackSignalHandler::Release();
	// Passing -1 will leave these restored and won't trap them
	if ((PTRINT)CrashHandler == -1)
	{
		return;
	}

	FFatalSignalHandler::Init();
	FThreadCallstackSignalHandler::Init();
#endif
}

bool FAndroidMisc::GetUseVirtualJoysticks()
{
	// joystick on commandline means don't require virtual joysticks
	if (FParse::Param(FCommandLine::Get(), TEXT("joystick")))
	{
		return false;
	}

	// Amazon Fire TV doesn't require virtual joysticks
	if (FAndroidMisc::GetDeviceMake() == FString("Amazon"))
	{
		if (FAndroidMisc::GetDeviceModel().StartsWith(TEXT("AFT")))
		{
			return false;
		}
	}

	// Oculus HMDs don't require virtual joysticks
	if (FAndroidMisc::GetDeviceMake() == FString("Oculus"))
	{
		return false;
	}

	return true;
}

bool FAndroidMisc::SupportsTouchInput()
{
	// Amazon Fire TV doesn't support touch input
	if (FAndroidMisc::GetDeviceMake() == FString("Amazon"))
	{
		if (FAndroidMisc::GetDeviceModel().StartsWith(TEXT("AFT")))
		{
			return false;
		}
	}

	// Oculus HMDs don't support touch input
	if (FAndroidMisc::GetDeviceMake() == FString("Oculus"))
	{
		return false;
	}

	return true;
}

const TCHAR* FAndroidMisc::GetDefaultDeviceProfileName()
{ 
	return ANDROID_DEFAULT_DEVICE_PROFILE_NAME; 
}

extern void AndroidThunkCpp_RegisterForRemoteNotifications();
extern void AndroidThunkCpp_UnregisterForRemoteNotifications();
extern bool AndroidThunkCpp_IsAllowedRemoteNotifications();

extern void AndroidThunkCpp_OnAudioFocusChangedListenersUpdated(bool bInHasListeners);

void FAndroidMisc::RegisterForRemoteNotifications()
{
#if USE_ANDROID_JNI
	AndroidThunkCpp_RegisterForRemoteNotifications();
#endif
}

void FAndroidMisc::UnregisterForRemoteNotifications()
{
#if USE_ANDROID_JNI
	AndroidThunkCpp_UnregisterForRemoteNotifications();
#endif
}

bool FAndroidMisc::IsAllowedRemoteNotifications()
{
#if USE_ANDROID_JNI
	return AndroidThunkCpp_IsAllowedRemoteNotifications();
#else
	return false;
#endif
}

FDelegateHandle FAndroidMisc::RegisterAudioFocusChangedEventListener(const FOnAudioFocusChangedDelegate Callback)
{
	if (!FCoreDelegates::AudioFocusChangedDelegate.IsBound())
	{
		AndroidThunkCpp_OnAudioFocusChangedListenersUpdated(true);
	}
	return FCoreDelegates::AudioFocusChangedDelegate.Add(Callback);
}

void FAndroidMisc::UnregisterAudioFocusChangedEventListener(const FDelegateHandle DelegateHandle)
{
	FCoreDelegates::AudioFocusChangedDelegate.Remove(DelegateHandle);
	if (!FCoreDelegates::AudioFocusChangedDelegate.IsBound())
	{
		AndroidThunkCpp_OnAudioFocusChangedListenersUpdated(false);
	}
}

#if USE_ANDROID_JNI
void JNICALL UE::Jni::FGameActivity::nativeOnAudioFocusChanged(JNIEnv* jenv, jobject thiz, jint FocusChange)
{
	if (FTaskGraphInterface::IsRunning())
	{
		FFunctionGraphTask::CreateAndDispatchWhenReady([FocusChange]()
		{
			FCoreDelegates::AudioFocusChangedDelegate.Broadcast(static_cast<EAndroidAudioFocusChange>(FocusChange) == EAndroidAudioFocusChange::Gain);
		}, TStatId(), NULL, ENamedThreads::GameThread);
	}
}
#endif

static FString PendingProtocolActivationUri;
static void HandleNewIntentUri_GameThread()
{
	static FTSTicker::FDelegateHandle ProtocolActivationTickerHandle;

	if (GAndroidBroadcastIntentData && !PendingProtocolActivationUri.IsEmpty())
	{
		if (!ProtocolActivationTickerHandle.IsValid())
		{
			ProtocolActivationTickerHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda(
				[](float)
				{
					if (FCoreDelegates::OnActivatedByProtocol.IsBound())
					{
						FCoreDelegates::OnActivatedByProtocol.Broadcast(PendingProtocolActivationUri, PLATFORMUSERID_NONE);

						PendingProtocolActivationUri.Reset();
						ProtocolActivationTickerHandle.Reset();
						return false; // remove from ticker
					}

					return true; // try again next tick
				}
			));
		}
	}
	else if (ProtocolActivationTickerHandle.IsValid())
	{
		FTSTicker::RemoveTicker(ProtocolActivationTickerHandle);
		ProtocolActivationTickerHandle.Reset();
	}
}

FString FAndroidMisc::GetPendingActivationProtocol()
{
	return PendingProtocolActivationUri;
}

void FAndroidMisc::HandleNewIntentUri(const FString& IntentUri)
{
	PendingProtocolActivationUri = IntentUri;
	
	// TaskGraph may be null if app was resumed before it's fully initialized, in such cases regular start will pick up the intent and send it again
	if (!PendingProtocolActivationUri.IsEmpty() && FTaskGraphInterface::IsRunning())
	{
		FFunctionGraphTask::CreateAndDispatchWhenReady([]()
			{
				HandleNewIntentUri_GameThread();
			}, TStatId(), NULL, ENamedThreads::GameThread);
	}
}

TArray<uint8> FAndroidMisc::GetSystemFontBytes()
{
#if USE_ANDROID_FILE
	TArray<uint8> FontBytes;
	static FString FullFontPath = GFontPathBase + FString("DroidSans.ttf");
	FFileHelper::LoadFileToArray(FontBytes, *FullFontPath);
	return FontBytes;
#else 
	return FGenericPlatformMisc::GetSystemFontBytes();
#endif
}

class IPlatformChunkInstall* FAndroidMisc::GetPlatformChunkInstall()
{
	static IPlatformChunkInstall* ChunkInstall = nullptr;
	static bool bIniChecked = false;
	if (!ChunkInstall || !bIniChecked)
	{
		FString ProviderName;
		IPlatformChunkInstallModule* PlatformChunkInstallModule = nullptr;
		if (!GEngineIni.IsEmpty())
		{
			FString InstallModule;
			GConfig->GetString(TEXT("StreamingInstall"), TEXT("DefaultProviderName"), InstallModule, GEngineIni);
			FModuleStatus Status;
			if (FModuleManager::Get().QueryModule(*InstallModule, Status))
			{
				PlatformChunkInstallModule = FModuleManager::LoadModulePtr<IPlatformChunkInstallModule>(*InstallModule);
				if (PlatformChunkInstallModule != nullptr)
				{
					// Attempt to grab the platform installer
					ChunkInstall = PlatformChunkInstallModule->GetPlatformChunkInstall();
				}
			}
			bIniChecked = true;
		}
		if (!ChunkInstall)
		{
			// Placeholder instance
			ChunkInstall = FGenericPlatformMisc::GetPlatformChunkInstall();
		}
	}

	return ChunkInstall;
}

void FAndroidMisc::PrepareMobileHaptics(EMobileHapticsType Type)
{
}

void FAndroidMisc::TriggerMobileHaptics()
{
#if USE_ANDROID_JNI
	extern void AndroidThunkCpp_Vibrate(int32 Intensity, int32 Duration);
	// directly play a small vibration one-shot
	// note: this will do nothing if device is already playing force feedback (non-zero intensity)
	// but will play and not be cancelled by force feedback since it only sends updates when not already above zero
	AndroidThunkCpp_Vibrate(255, 10);
#endif
}

void FAndroidMisc::ReleaseMobileHaptics()
{

}

void FAndroidMisc::ShareURL(const FString& URL, const FText& Description, int32 LocationHintX, int32 LocationHintY)
{
#if USE_ANDROID_JNI
	extern void AndroidThunkCpp_ShareURL(const FString& URL, const FText& Description, const FText& SharePrompt, int32 LocationHintX, int32 LocationHintY);
	AndroidThunkCpp_ShareURL(URL, Description, NSLOCTEXT("AndroidMisc", "ShareURL", "Share URL"), LocationHintX, LocationHintY);
#endif
}

FString FAndroidMisc::LoadTextFileFromPlatformPackage(const FString& RelativePath)
{
#if USE_ANDROID_JNI
	AAssetManager* AssetMgr = AndroidThunkCpp_GetAssetManager();
	AAsset* asset = AAssetManager_open(AssetMgr, TCHAR_TO_UTF8(*RelativePath), AASSET_MODE_BUFFER);

	if (asset)
	{
		const void* FileContents = (const ANSICHAR*)AAsset_getBuffer(asset);
		int32 FileLength = AAsset_getLength(asset);

		TArray<ANSICHAR> TextContents;
		TextContents.AddUninitialized(FileLength + 1);
		FMemory::Memcpy(TextContents.GetData(), FileContents, FileLength);
		TextContents[FileLength] = 0;

		AAsset_close(asset);

		return FString(ANSI_TO_TCHAR(TextContents.GetData()));
	}
#endif
	return FString();
}

bool FAndroidMisc::FileExistsInPlatformPackage(const FString& RelativePath)
{
#if USE_ANDROID_JNI
	if (!FPathViews::IsRelativePath(RelativePath))
	{
		UE_LOGF(LogAndroid, Warning, "FileExistsInPlatformPackage: expected a relative path, but received %ls", *RelativePath);
		return false;
	}

	AAssetManager* AssetMgr = AndroidThunkCpp_GetAssetManager();
	AAsset* asset = AAssetManager_open(AssetMgr, TCHAR_TO_UTF8(*RelativePath), AASSET_MODE_UNKNOWN);
	if (asset)
	{
		AAsset_close(asset);
		return true;
	}
#endif
	return false;
}

void FAndroidMisc::SetVersionInfo( FString InAndroidVersion, int32 InTargetSDKVersion, FString InDeviceMake, FString InDeviceModel, FString InDeviceBuildNumber, FString InOSLanguage, FString InProductName)
{
	AndroidVersion = InAndroidVersion;
	AndroidMajorVersion = FCString::Atoi(*InAndroidVersion);
	TargetSDKVersion = InTargetSDKVersion;
	DeviceMake = InDeviceMake;
	DeviceModel = InDeviceModel;
	DeviceBuildNumber = InDeviceBuildNumber;
	OSLanguage = InOSLanguage;
	ProductName = InProductName;
	UE_LOGF(LogAndroid, Display, "Android Version: %ls, Make: %ls, Model: %ls, BuildNumber: %ls, Language: %ls, Product name: %ls", *AndroidVersion, *DeviceMake, *DeviceModel, *DeviceBuildNumber, *OSLanguage, ProductName.IsEmpty() ? TEXT("[not set]") : *ProductName);
}

const FString FAndroidMisc::GetAndroidVersion()
{
	return AndroidVersion;
}

int32 FAndroidMisc::GetAndroidMajorVersion()
{
	return AndroidMajorVersion;
}

int32 FAndroidMisc::GetTargetSDKVersion()
{
	return TargetSDKVersion;
}

const FString FAndroidMisc::GetDeviceMake()
{
	return DeviceMake;
}

const FString FAndroidMisc::GetDeviceModel()
{
	return DeviceModel;
}

const FString FAndroidMisc::GetDeviceBuildNumber()
{
	return DeviceBuildNumber;
}

const FString FAndroidMisc::GetOSLanguage()
{
	return OSLanguage;
}

const FString FAndroidMisc::GetProductName()
{
	return ProductName;
}

const FString FAndroidMisc::GetProjectVersion() {
	return FString::FromInt(GAndroidPackageVersion);
}

FString FAndroidMisc::GetDefaultLocale()
{
	return OSLanguage;
}

bool FAndroidMisc::GetVolumeButtonsHandledBySystem()
{
	return VolumeButtonsHandledBySystem;
}

void FAndroidMisc::SetVolumeButtonsHandledBySystem(bool enabled)
{
	VolumeButtonsHandledBySystem = enabled;
}

#if USE_ANDROID_JNI
int32 FAndroidMisc::GetAndroidBuildVersion()
{
	if (AndroidBuildVersion > 0)
	{
		return AndroidBuildVersion;
	}
	if (AndroidBuildVersion <= 0)
	{
		JNIEnv* JEnv = AndroidJavaEnv::GetJavaEnv();
		if (nullptr != JEnv)
		{
			jclass Class = AndroidJavaEnv::FindJavaClassGlobalRef("com/epicgames/unreal/GameActivity");
			if (nullptr != Class)
			{
				jfieldID Field = JEnv->GetStaticFieldID(Class, "ANDROID_BUILD_VERSION", "I");
				if (nullptr != Field)
				{
					AndroidBuildVersion = JEnv->GetStaticIntField(Class, Field);
				}
				JEnv->DeleteGlobalRef(Class);
			}
		}
	}
	return AndroidBuildVersion;
}
#endif

static bool bForceUnsupported = false;
void FAndroidMisc::SetForceUnsupported(bool bInOverride)
{
	bForceUnsupported = bInOverride;
}

#if USE_ANDROID_JNI
bool FAndroidMisc::IsSupportedAndroidDevice()
{
	static bool bChecked = false;
	static bool bSupported = true;

	if (!bChecked)
	{
		bChecked = true;

		JNIEnv* JEnv = AndroidJavaEnv::GetJavaEnv();
		if (nullptr != JEnv)
		{
			jclass Class = AndroidJavaEnv::FindJavaClassGlobalRef("com/epicgames/unreal/GameActivity");
			if (nullptr != Class)
			{
				jfieldID Field = JEnv->GetStaticFieldID(Class, "bSupportedDevice", "Z");
				if (nullptr != Field)
				{
					bSupported = (bool)JEnv->GetStaticBooleanField(Class, Field);
				}
				JEnv->DeleteGlobalRef(Class);
			}
		}
	}
	return bForceUnsupported ? false : bSupported;
}
#else
bool FAndroidMisc::IsSupportedAndroidDevice()
{
	return !bForceUnsupported;
}
#endif

enum class EDeviceVulkanSupportStatus
{
	Uninitialized,
	NotSupported,
	Supported
};

static FString VulkanVersionString;
static FString VulkanDeviceNameString;
static EDeviceVulkanSupportStatus VulkanSupport = EDeviceVulkanSupportStatus::Uninitialized;

///////////////////////////////////////////////////////////////////////////////
//
// Extracted from vk_platform.h and vulkan.h with modifications just to allow
// vkCreateInstance/vkDestroyInstance to be called to check if a driver is actually
// available (presence of libvulkan.so only means it may be available, not that
// there is an actual usable one). Cannot wait for VulkanRHI init to do this (too
// late) and vulkan.h header not guaranteed to be available. This part of the header
// is unlikely to change in future so safe enough to use this truncated version.
//
// Namespace here to avoid conflicts with real headers
namespace AndroidPlatformMisc
{
#if PLATFORM_ANDROID_ARM
// On Android/ARMv7a, Vulkan functions use the armeabi-v7a-hard calling
#define VKAPI_ATTR __attribute__((pcs("aapcs-vfp")))
#define VKAPI_CALL
#define VKAPI_PTR  VKAPI_ATTR
#else
// On other platforms, use the default calling convention
#define VKAPI_ATTR
#define VKAPI_CALL
#define VKAPI_PTR
#endif

///////////////////////////////////////////////////////////////////////////////

#define UE_VK_API_VERSION	VK_MAKE_VERSION(1, 1, 0)

// Only valid if EDeviceVulkanSupportStatus is EDeviceVulkanSupportStatus::Supported
VkPhysicalDeviceProperties VulkanPhysicalDeviceProperties;

static EDeviceVulkanSupportStatus AttemptVulkanInit(void* VulkanLib)
{
	if (VulkanLib == nullptr)
	{
		return EDeviceVulkanSupportStatus::NotSupported;
	}

	// Try to get required functions to check for driver
	PFN_vkCreateInstance vkCreateInstance = (PFN_vkCreateInstance)dlsym(VulkanLib, "vkCreateInstance");
	PFN_vkDestroyInstance vkDestroyInstance = (PFN_vkDestroyInstance)dlsym(VulkanLib, "vkDestroyInstance");
	PFN_vkEnumeratePhysicalDevices vkEnumeratePhysicalDevices = (PFN_vkEnumeratePhysicalDevices)dlsym(VulkanLib, "vkEnumeratePhysicalDevices");
	PFN_vkGetPhysicalDeviceProperties vkGetPhysicalDeviceProperties = (PFN_vkGetPhysicalDeviceProperties)dlsym(VulkanLib, "vkGetPhysicalDeviceProperties");
	PFN_vkEnumerateDeviceExtensionProperties vkEnumerateDeviceExtensionProperties = (PFN_vkEnumerateDeviceExtensionProperties)dlsym(VulkanLib, "vkEnumerateDeviceExtensionProperties");

	if (!vkCreateInstance || !vkDestroyInstance || !vkEnumeratePhysicalDevices || !vkGetPhysicalDeviceProperties || !vkEnumerateDeviceExtensionProperties)
	{
		UE_LOGF(LogAndroid, Log, "Vulkan not supported: vkCreateInstance: 0x%p, vkDestroyInstance: 0x%p, vkEnumeratePhysicalDevices: 0x%p, vkGetPhysicalDeviceProperties: 0x%p, vkEnumerateDeviceExtensionProperties: 0x%p", vkCreateInstance, vkDestroyInstance, vkEnumeratePhysicalDevices, vkGetPhysicalDeviceProperties, vkEnumerateDeviceExtensionProperties);
		return EDeviceVulkanSupportStatus::NotSupported;
	}

	// Try to create instance to verify driver available. Logic here matches that in VulkanRHI.cpp.
	// Drivers need to detect the app and potentially return VK_ERROR_INCOMPATIBLE_DRIVER to disable Vulkan support for known incompatibility
	auto* CVarDisableEngineAndAppRegistration = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.DisableEngineAndAppRegistration"));
	bool bDisableEngineRegistration = (CVarDisableEngineAndAppRegistration && CVarDisableEngineAndAppRegistration->GetValueOnAnyThread() != 0);

	FString EngineName = FApp::GetEpicProductIdentifier() + FEngineVersion::Current().ToString(EVersionComponent::Minor);
	FTCHARToUTF8 EngineNameConverter(*EngineName);
	FTCHARToUTF8 ProjectNameConverter(FApp::GetProjectName());

	VkApplicationInfo AppInfo;
	FMemory::Memzero(AppInfo);
	AppInfo.sType = VK_STRUCTURE_TYPE_APPLICATION_INFO;
	AppInfo.pApplicationName = bDisableEngineRegistration ? nullptr : ProjectNameConverter.Get();
	AppInfo.applicationVersion = static_cast<uint32>(BuildSettings::GetCurrentChangelist()) | (BuildSettings::IsLicenseeVersion() ? 0x80000000 : 0);
	AppInfo.pEngineName = bDisableEngineRegistration ? nullptr : EngineNameConverter.Get();
	AppInfo.engineVersion = FEngineVersion::Current().GetMinor();
	AppInfo.apiVersion = UE_VK_API_VERSION;

	VkInstanceCreateInfo InstInfo;
	FMemory::Memzero(InstInfo);
	InstInfo.sType = VK_STRUCTURE_TYPE_INSTANCE_CREATE_INFO;
	InstInfo.pNext = nullptr;
	InstInfo.pApplicationInfo = &AppInfo;
	InstInfo.enabledExtensionCount = 0;
	InstInfo.ppEnabledExtensionNames = nullptr;

	VkInstance Instance;
	VkResult Result = vkCreateInstance(&InstInfo, nullptr, &Instance);
	if (Result != VK_SUCCESS)
	{
		return EDeviceVulkanSupportStatus::NotSupported;
	}

	// Determine Vulkan device's API level.
	uint32 GpuCount = 0;
	Result = vkEnumeratePhysicalDevices(Instance, &GpuCount, nullptr);
	if (Result != VK_SUCCESS || GpuCount == 0)
	{
		vkDestroyInstance(Instance, nullptr);
		return EDeviceVulkanSupportStatus::NotSupported;
	}

	TArray<VkPhysicalDevice> PhysicalDevices;
	PhysicalDevices.AddZeroed(GpuCount);
	Result = vkEnumeratePhysicalDevices(Instance, &GpuCount, PhysicalDevices.GetData());
	if (Result != VK_SUCCESS)
	{
		vkDestroyInstance(Instance, nullptr);
		return EDeviceVulkanSupportStatus::NotSupported;
	}

	// Don't care which device - This code is making the assumption that all devices will have same api version.
	vkGetPhysicalDeviceProperties(PhysicalDevices[0], &VulkanPhysicalDeviceProperties);

	//for now we are allowing devices without the timing extension to run with a basic CPU frame pacer.
#if 0
	if (FAndroidPlatformRHIFramePacer::CVarAllowFrameTimestamps.GetValueOnAnyThread() != 0)
	{
		bool bHasVKGoogleDisplayTiming = false;
		{
			TArray<VkExtensionProperties> ExtensionProps;
			do
			{
				uint32 Count = 0;
				Result = vkEnumerateDeviceExtensionProperties(PhysicalDevices[0], nullptr, &Count, nullptr);
				check(Result >= VK_SUCCESS);

				if (Count > 0)
				{
					ExtensionProps.Empty(Count);
					ExtensionProps.AddUninitialized(Count);
					Result = vkEnumerateDeviceExtensionProperties(PhysicalDevices[0], nullptr, &Count, ExtensionProps.GetData());
					check(Result >= VK_SUCCESS);
				}
			} while (Result == VK_INCOMPLETE);		
			check(Result >= VK_SUCCESS);

			
			for (int32 i = 0; i < ExtensionProps.Num(); ++i)
			{
				UE_LOGF(LogAndroid, Log, "Checking extension: %ls.", ANSI_TO_TCHAR(ExtensionProps[i].extensionName));
				if (!FCStringAnsi::Strcmp(VK_GOOGLE_DISPLAY_TIMING_EXTENSION_NAME, ExtensionProps[i].extensionName))
				{
					bHasVKGoogleDisplayTiming = true;
					break;
				}
			}
		}
		if (!bHasVKGoogleDisplayTiming)
		{
			vkDestroyInstance(Instance, nullptr);

			UE_LOGF(LogAndroid, Log, "Vulkan not supported, cannot find VK_GOOGLE_display_timing extension.");
			return EDeviceVulkanSupportStatus::NotSupported;
		}
	}
#endif

	VulkanVersionString = FString::Printf(TEXT("%d.%d.%d"), VK_VERSION_MAJOR(VulkanPhysicalDeviceProperties.apiVersion), VK_VERSION_MINOR(VulkanPhysicalDeviceProperties.apiVersion), VK_VERSION_PATCH(VulkanPhysicalDeviceProperties.apiVersion));
	vkDestroyInstance(Instance, nullptr);
	VulkanDeviceNameString = VulkanPhysicalDeviceProperties.deviceName;

	return EDeviceVulkanSupportStatus::Supported;
}

} // Namespace


bool FAndroidMisc::HasVulkanDriverSupport()
{
#if !USE_ANDROID_JNI
	VulkanSupport = EDeviceVulkanSupportStatus::NotSupported;
	VulkanVersionString = TEXT("0.0.0");
#else
	// this version does not check for VulkanRHI or disabled by cvars!
	if (VulkanSupport == EDeviceVulkanSupportStatus::Uninitialized)
	{
		// assume no
		VulkanSupport = EDeviceVulkanSupportStatus::NotSupported;
		VulkanVersionString = TEXT("0.0.0");

		// check for libvulkan.so
		void* VulkanLib = dlopen("libvulkan.so", RTLD_NOW | RTLD_LOCAL);
		if (VulkanLib != nullptr)
		{
			UE_LOGF(LogAndroid, Log, "Vulkan library detected, checking for available driver");

			// if Nougat, we can check the Vulkan version
			if (FAndroidMisc::GetAndroidBuildVersion() >= 24)
			{
				extern int32 AndroidThunkCpp_GetMetaDataInt(const FString& Key);
				int32 VulkanVersion = AndroidThunkCpp_GetMetaDataInt(TEXT("android.hardware.vulkan.version"));
				if (VulkanVersion >= UE_VK_API_VERSION)
				{
					// final check, try initializing the instance
					VulkanSupport = AndroidPlatformMisc::AttemptVulkanInit(VulkanLib);
				}
			}
			else
			{
				// otherwise, we need to try initializing the instance
				VulkanSupport = AndroidPlatformMisc::AttemptVulkanInit(VulkanLib);
			}

			dlclose(VulkanLib);

			if (VulkanSupport == EDeviceVulkanSupportStatus::Supported)
			{
				UE_LOGF(LogAndroid, Log, "VulkanRHI is available, Vulkan capable device detected.");
				return true;
			}
			else
			{
				UE_LOGF(LogAndroid, Log, "Vulkan driver NOT available.");
			}
		}
		else
		{
			UE_LOGF(LogAndroid, Log, "Vulkan library NOT detected.");
		}
	}
#endif
	return VulkanSupport == EDeviceVulkanSupportStatus::Supported;
}

// Test for device vulkan support.
static void EstablishVulkanDeviceSupport()
{
	// just do this check once
	if (VulkanSupport == EDeviceVulkanSupportStatus::Uninitialized)
	{
		// this call will initialize VulkanSupport
		FAndroidMisc::HasVulkanDriverSupport();
	}
}

bool FAndroidMisc::IsDesktopVulkanAvailable()
{
	static int CachedDesktopVulkanAvailable = -1;

	if (CachedDesktopVulkanAvailable == -1)
	{
		CachedDesktopVulkanAvailable = 0;

		bool bSupportsVulkanSM5 = false;

		GConfig->GetBool(TEXT("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings"), TEXT("bSupportsVulkanSM5"), bSupportsVulkanSM5, GEngineIni);

		if (bSupportsVulkanSM5)
		{
			CachedDesktopVulkanAvailable = 1;
		}
	}

	return CachedDesktopVulkanAvailable == 1;
}

bool FAndroidMisc::IsVulkanAvailable()
{
	check(VulkanSupport != EDeviceVulkanSupportStatus::Uninitialized);

	static int CachedVulkanAvailable = -1;
	if (CachedVulkanAvailable == -1)
	{
		CachedVulkanAvailable = 0;
		if (VulkanSupport == EDeviceVulkanSupportStatus::Supported)
		{
			bool bSupportsVulkan = false;
			GConfig->GetBool(TEXT("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings"), TEXT("bSupportsVulkan"), bSupportsVulkan, GEngineIni);

			// whether to detect Vulkan by default or require the -detectvulkan command line parameter
			bool bDetectVulkanByDefault = true;
			GConfig->GetBool(TEXT("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings"), TEXT("bDetectVulkanByDefault"), bDetectVulkanByDefault, GEngineIni);
			const bool bDetectVulkanCmdLine = FParse::Param(FCommandLine::Get(), TEXT("detectvulkan"));

			const bool bSupportsVulkanSM5 = IsDesktopVulkanAvailable();

			const bool bVulkanDisabledCmdLine = FParse::Param(FCommandLine::Get(), TEXT("GL")) || FParse::Param(FCommandLine::Get(), TEXT("OpenGL"));

			if (!FModuleManager::Get().ModuleExists(TEXT("VulkanRHI")))
			{
				UE_LOGF(LogAndroid, Log, "Vulkan not available as VulkanRHI not present.");
			}
			else if (!(bSupportsVulkan || bSupportsVulkanSM5))
			{
				UE_LOGF(LogAndroid, Log, "Vulkan not available as project packaged without bSupportsVulkan or bSupportsVulkanSM5.");
			}
			else if (bVulkanDisabledCmdLine)
			{
				UE_LOGF(LogAndroid, Log, "Vulkan API detection is disabled by a command line option.");
			}
            else if (!bDetectVulkanByDefault && !bDetectVulkanCmdLine)
			{
				UE_LOGF(LogAndroid, Log, "Vulkan available but detection disabled by bDetectVulkanByDefault=False in AndroidRuntimeSettings. Use -detectvulkan to override.");
			}
			else
			{
				CachedVulkanAvailable = 1;
			}
		}
	}

	return CachedVulkanAvailable == 1;
}

bool FAndroidMisc::ShouldUseVulkan()
{
	check(VulkanSupport != EDeviceVulkanSupportStatus::Uninitialized);
	static int CachedShouldUseVulkan = -1;

	if (CachedShouldUseVulkan == -1)
	{
		CachedShouldUseVulkan = 0;

		static const auto CVarDisableVulkan = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Android.DisableVulkanSupport"));

		const bool bVulkanAvailable = IsVulkanAvailable();

		const bool bVulkanDisabledCVar = CVarDisableVulkan->GetValueOnAnyThread() == 1;

		if (bVulkanAvailable && !bVulkanDisabledCVar)
		{
			CachedShouldUseVulkan = 1;
			UE_LOGF(LogAndroid, Log, "VulkanRHI will be used!");
		}
		else
		{
			UE_LOGF(LogAndroid, Log, "VulkanRHI will NOT be used:");
			if (!bVulkanAvailable)
			{
				UE_LOGF(LogAndroid, Log, " ** Vulkan support is not available (Driver, RHI or shaders are missing, or disabled by cmdline, see above logging for details)");
			}
			if (bVulkanDisabledCVar)
			{
				UE_LOGF(LogAndroid, Log, " ** Vulkan is disabled via console variable.");
			}
			UE_LOGF(LogAndroid, Log, "OpenGL ES will be used.");
		}
	}

	return CachedShouldUseVulkan == 1;
}

bool FAndroidMisc::ShouldUseDesktopVulkan()
{
	static int CachedShouldUseDesktopVulkan = -1;

	if (CachedShouldUseDesktopVulkan == -1)
	{
		CachedShouldUseDesktopVulkan = 0;

		const bool bVulkanSM5Enabled = IsDesktopVulkanAvailable();

		static const auto CVarDisableVulkanSM5 = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Android.DisableVulkanSM5Support"));
		const bool bVulkanSM5Disabled = CVarDisableVulkanSM5->GetValueOnAnyThread() == 1;

		if (bVulkanSM5Enabled && !bVulkanSM5Disabled)
		{
			CachedShouldUseDesktopVulkan = 1;
			UE_LOGF(LogAndroid, Log, "Vulkan SM5 RHI will be used!");
		}
		else if(bVulkanSM5Disabled)
		{
			UE_LOGF(LogAndroid, Log, "Vulkan SM5 is available but disabled for this device.");
		}
		else if (!bVulkanSM5Enabled)
		{
			UE_LOGF(LogAndroid, Log, "** Vulkan SM5 support is not available (Driver, RHI or shaders are missing, or disabled by cmdline, see above logging for details)");
		}
	}

	return CachedShouldUseDesktopVulkan;
}

FString FAndroidMisc::GetVulkanVersion()
{
	check(VulkanSupport != EDeviceVulkanSupportStatus::Uninitialized);
	return VulkanVersionString;
}

FString FAndroidMisc::GetVulkanDeviceNameString()
{
	check(VulkanSupport != EDeviceVulkanSupportStatus::Uninitialized);
	return VulkanDeviceNameString;
}

FString FAndroidMisc::GetVKQualityRecommendation()
{
	static bool bCachedResult = false;
	static FString Result;

#if USE_ANDROID_JNI
	if (bCachedResult)
	{
		return Result;
	}

	if (VulkanSupport == EDeviceVulkanSupportStatus::Supported)
	{
		if (AndroidVKQuality::LoadVkQuality(TCHAR_TO_UTF8(*FAndroidMisc::GetGLVersion()), &AndroidPlatformMisc::VulkanPhysicalDeviceProperties))
		{
			Result = AndroidVKQuality::GetVKQualityRecommendation();
			bCachedResult = true;
			AndroidVKQuality::UnloadVkQuality();
		}
	}
#endif
	return Result;
}

bool FAndroidMisc::IsExternalMemoryAndroidHardwareBufferExtensionLoaded()
{
	if (GIsRHIInitialized && ShouldUseVulkan())
	{
		IVulkanDynamicRHI* RHI = GetIVulkanDynamicRHI();
		TArray<FAnsiString> LoadedDeviceExtensions = RHI->RHIGetLoadedDeviceExtensions();

		// Consider replace the following check by using PhysicalDeviceFeatures.Core_1_1.samplerYcbcrConversion
		if (!LoadedDeviceExtensions.Find(FAnsiString(VK_KHR_SAMPLER_YCBCR_CONVERSION_EXTENSION_NAME)))
		{
			UE_LOGF(LogAndroid, Log, "VK_KHR_SAMPLER_YCBCR_CONVERSION_EXTENSION is not available!");
		}
		else if (!LoadedDeviceExtensions.Find(FAnsiString(VK_ANDROID_EXTERNAL_MEMORY_ANDROID_HARDWARE_BUFFER_EXTENSION_NAME)))
		{
			UE_LOGF(LogAndroid, Log, "VK_ANDROID_EXTERNAL_MEMORY_ANDROID_HARDWARE_BUFFER_EXTENSION is not available!");
		}
		else
		{
			UE_LOGF(LogAndroid, Log, "VK_ANDROID_EXTERNAL_MEMORY_ANDROID_HARDWARE_BUFFER_EXTENSION is loaded.");
			return true;
		}
	}

	return false;
}

static std::atomic_bool bConfigRulesReady = false;

const TMap<FString, FString>& FAndroidMisc::GetConfigRuleVars()
{
	if (!ensure(bConfigRulesReady))
	{
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Configrule vars requested too early. Using empty data."));
		// If you encounter this, change the call to use 'static FUtf8StringView blah = GetConfigRulesVariableAsUtf8("blah");'
		static TMap<FString, FString> EmptyCvars;
		return EmptyCvars;
	}
	
	// ConfigRuleVars are read-only from here on.
	return ConfigRulesVariables;
}

namespace AndroidPlatformMisc
{
	// Fills Buffer with the human-readable string for Error via strerror.
	// Returns "" if Error is 0.
	static const TCHAR* GetErrnoString(int Error, TCHAR (&Buffer)[256])
	{
		if (Error == 0)
		{
			Buffer[0] = TEXT('\0');
		}
		else
		{
			FCString::Snprintf(Buffer, UE_ARRAY_COUNT(Buffer), TEXT("%hs"), strerror(Error));
		}
		return Buffer;
	}

 	static UE::FSharedMutex EarlyConfigRuleReaderLock;

	class FEarlyConfigRuleReader
	{
		static constexpr double SocketReadTimeoutS = 1.5;
		
		// \0 denotes abstract unix domain socket name.
		// This matches java side EarlyConfigrulesSetup in game activity.
		// java local sockets bind to the abstract domain.
		static constexpr char SocketName[] = "\0ue.earlyconfigrules";
		static inline UTF8CHAR EarlyConfRuleBuff[16384] = { 0 };
		static inline int EarlyConfRuleBuffSize = 0;

		void SetBuffer(const TArrayView<UTF8CHAR> Buff)
		{
			// the buffer should only ever be set one time, therefore no memzero required.
			if(Buff.NumBytes() <= sizeof(EarlyConfRuleBuff))
			{
				FMemory::Memcpy(EarlyConfRuleBuff, Buff.GetData(), Buff.NumBytes());
				EarlyConfRuleBuffSize = Buff.NumBytes();
			}
			else
			{
				FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FEarlyConfigRuleReader: Failed to set configrules from buffer of size %d, buffer would overflow. (max size %zu)"), Buff.NumBytes(), sizeof(EarlyConfRuleBuff));
			}
		}

		void ReadConfigRulesFromSocket()
		{
			FPlatformMisc::BeginNamedEvent(FColor(0), TEXT("Start config reader"));
			ON_SCOPE_EXIT
			{
				FPlatformMisc::EndNamedEvent();
			};

			double t0 = FPlatformTime::Seconds();
			double tConnected = 0, tVerified = 0, tReceived = 0;
			int listenfd = socket(AF_UNIX, SOCK_STREAM, 0);
			TCHAR ErrBuf[256];
			if (listenfd < 0)
			{
				FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FEarlyConfigRuleReader: Failed. socket open failed (%d) %s"), errno, GetErrnoString(errno, ErrBuf));
				return;
			}

			ON_SCOPE_EXIT
			{
				close(listenfd);
			};

			struct sockaddr_un addr = { 0 };
			addr.sun_family = AF_UNIX;
			// size-1 because abstract socket names are *not* null terminated
			memcpy(addr.sun_path, SocketName, sizeof(SocketName) - 1);

			int result = -1;
			while ((FPlatformTime::Seconds() - t0) < SocketReadTimeoutS)
			{
				result = connect(listenfd, (struct sockaddr*)&addr, sizeof(addr.sun_family) + sizeof(SocketName) - 1);
				if (result == 0)
				{
					break;
				}

				if (errno == ECONNREFUSED || errno == ENOENT)
				{
					// socket not created yet?
					FPlatformProcess::SleepNoStats(0.001f);
				}
				else
				{
					// any other error == failed.
					FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FEarlyConfigRuleReader: Failed. socket connect failed (%d) %s"), errno, GetErrnoString(errno, ErrBuf));
					return;
				}
			}

			if (result != 0)
			{
				FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FEarlyConfigRuleReader: Failed. Timeout when waiting for early rules. %fs)"), (float)(FPlatformTime::Seconds() - t0));
				return;
			}

			tConnected = FPlatformTime::Seconds();

			// Abstract Unix sockets are globally visible. Verify the peer UID matches our own
			// to reject connections from other apps. UID is sufficient on Android as each app
			// has a unique UID; processes sharing our UID (e.g. app child services) already
			// have equivalent access to our files and SharedPreferences.
			// SO_PEERCRED is kernel-populated and cannot be spoofed.
			struct ucred peercred = {};
			socklen_t credlen = sizeof(peercred);
			if (getsockopt(listenfd, SOL_SOCKET, SO_PEERCRED, &peercred, &credlen) != 0)
			{
				FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FEarlyConfigRuleReader: Failed to get peer credentials (%d) %s"), errno, GetErrnoString(errno, ErrBuf));
				return;
			}
			if (peercred.uid != static_cast<uid_t>(getuid()))
			{
				FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FEarlyConfigRuleReader: Rejecting socket - peer UID %d does not match ours %d"), peercred.uid, getuid());
				return;
			}

			tVerified = FPlatformTime::Seconds();

			// set the timeout for recv to use the remaining time, keep a floor of 100ms incase we actually do have something ready.
			double RemainingTime = FMath::Max(0.1, SocketReadTimeoutS - (FPlatformTime::Seconds() - t0));
			struct timeval tv = { 0 };
			tv.tv_sec = (time_t)(RemainingTime);
			tv.tv_usec = ((time_t)(RemainingTime *1000)-((time_t)(RemainingTime) * 1000)) *1000;
			if (setsockopt(listenfd, SOL_SOCKET, SO_RCVTIMEO, (char*)&tv, sizeof(tv)))
			{
				FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FEarlyConfigRuleReader: Failed to set socket recv timeout (%d) %s"), errno, GetErrnoString(errno, ErrBuf));
			}

			bool bReading = true;
			do
			{
				int Remaining = sizeof(EarlyConfRuleBuff) - EarlyConfRuleBuffSize;
				if (Remaining <= 0)
				{
					// test for more, do not accept truncated results.
					char TestFomo;
					if (recv(listenfd, &TestFomo, sizeof(TestFomo), MSG_DONTWAIT) == 1)
					{
						// failed, buffer would overflow.
						EarlyConfRuleBuffSize = 0;
						FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FEarlyConfigRuleReader: Failed. Rejecting early configrules data. Buffer would overflow."));
					}
					bReading = false;
					break;
				}
				int readamount = recv(listenfd, EarlyConfRuleBuff + EarlyConfRuleBuffSize, Remaining, 0);
				if (readamount > 0)
				{
					EarlyConfRuleBuffSize += readamount;
				}
				else if (readamount == 0)
				{
					// socket closed. all done.
					bReading = false;
				}
				else if (errno != EINTR)
				{
					FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FEarlyConfigRuleReader: Failed to receive (%d) %s"), errno, GetErrnoString(errno, ErrBuf));
					EarlyConfRuleBuffSize = 0;
					bReading = false;
				}
			} while (bReading);

			tReceived = FPlatformTime::Seconds();

			if (EarlyConfRuleBuffSize > 0)
			{
				FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FEarlyConfigRuleReader: Success. %d bytes, connect=%.1fms verify=%.1fms recv=%.1fms total=%.1fms"),
					EarlyConfRuleBuffSize,
					(float)(tConnected - t0) * 1000.0f,
					(float)(tVerified - tConnected) * 1000.0f,
					(float)(tReceived - tVerified) * 1000.0f,
					(float)(tReceived - t0) * 1000.0f);
			}
			else
			{
				FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FEarlyConfigRuleReader: Failed. Did not receive early configrules (%d) %s"), errno, GetErrnoString(errno, ErrBuf));
			}
		}

	public:

		FEarlyConfigRuleReader(const TArrayView<UTF8CHAR> Utf8Buff)
		{
			// if we have a buffer then avoid waiting for sockets.
			if (Utf8Buff.IsEmpty())
			{
				ReadConfigRulesFromSocket();
			}
			else
			{
				SetBuffer(Utf8Buff);
			}
		}

		// If the socket failed (timeout etc) then an attempt is made to populate during nativeSetConfigRules.
		// returns true if the buffer was set.
		// because this is called outside of the static constructor we must use EarlyConfigRuleReaderLock.
		bool SetBufferIfEmpty(const TArrayView<UTF8CHAR> Buff)
		{
 			UE::TUniqueLock Lock(EarlyConfigRuleReaderLock);
			if (EarlyConfRuleBuffSize == 0 && Buff.NumBytes() <= sizeof(EarlyConfRuleBuff))
			{
				SetBuffer(Buff);
				return true;
			}
			return false;
		}

		// This is safe as the buffer is forever RO once set.
		bool GetBuffer(const UTF8CHAR*& BufferOUT, int& BufferSizeOUT) const
		{
 			UE::TSharedLock Lock(EarlyConfigRuleReaderLock);
			if (EarlyConfRuleBuffSize > 0)
			{
				BufferOUT = &EarlyConfRuleBuff[0];
				BufferSizeOUT = EarlyConfRuleBuffSize;
				return true;
			}
			return false;
		}
	};

	// on the first call we will either wait for a socket or use the supplied buffer.
	// this avoids slower socket path if we get to nativeSetConfigRules first.
	static FEarlyConfigRuleReader& GetConfigRuleReader(const TArrayView<UTF8CHAR> Utf8Buff = TArrayView<UTF8CHAR>())
	{
		static FEarlyConfigRuleReader EarlyConfigRuleReader(Utf8Buff);
		return EarlyConfigRuleReader;
	}
} // namespace AndroidPlatformMisc

// Note: This is a linear search through the buffer. 
// TODO: cache the stringviews once the allocator is available.
const FUtf8StringView FAndroidMisc::GetConfigRulesVariableAsUtf8(const FUtf8StringView Key)
{
	AndroidPlatformMisc::FEarlyConfigRuleReader& EarlyConfigRuleReader = AndroidPlatformMisc::GetConfigRuleReader();

	UTF8CHAR const* BuffString;
	int BuffSize;
	if (EarlyConfigRuleReader.GetBuffer(BuffString, BuffSize))
	{
		FUtf8StringView FoundKey;
		for (int Position = 0; Position<BuffSize;)
		{
			uint8 StrSize = BuffString[Position++];
			// check we have space to read the string.
			if (StrSize + Position > BuffSize)
			{
				break;
			}
			FUtf8StringView FoundStr(&BuffString[Position], StrSize);
			if (!FoundKey.GetData())
			{
				FoundKey = FoundStr;
			}
			else
			{
				if (FoundKey.Equals(Key))
				{
					return FoundStr;
				}
				FoundKey.Reset();
			}
			Position += StrSize;
		}
	}
	return FUtf8StringView();
}

bool FAndroidMisc::AllowThreadHeartBeat()
{
	static uint32 AllowThreadHeartBeatOnce = -1;
	if (AllowThreadHeartBeatOnce == -1)
	{
		const FString* AllowThreadHeartBeatConfigVar = GetConfigRulesVariable(TEXT("EnableThreadHeartBeat"));
		if (AllowThreadHeartBeatConfigVar)
		{
			AllowThreadHeartBeatOnce = (uint32)AllowThreadHeartBeatConfigVar->Equals("true", ESearchCase::IgnoreCase);
		}
		else
		{
			AllowThreadHeartBeatOnce = 0;
		}
	}

	return AllowThreadHeartBeatOnce == 1;
}

#if USE_ANDROID_JNI
void JNICALL UE::Jni::FGameActivity::nativeSetConfigRulesVariables(JNIEnv* jenv, jobject thiz, Java::Lang::TArray<jstring>* KeyValuePairs, Java::Lang::TArray<jbyte>* Utf8Vars)
{
	FPlatformMisc::BeginNamedEvent(FColor(0), TEXT("nativeSetConfigRulesVariables"));
	ON_SCOPE_EXIT
	{
		FPlatformMisc::EndNamedEvent();
	};

	verify(KeyValuePairs != nullptr);
	int32 JArrayCount = jenv->GetArrayLength(KeyValuePairs);
	int32 Count = FMath::DivideAndRoundDown(JArrayCount, 2) * 2; // ensure count is even to match our KV pair assumption later.
	UE_CLOGF(JArrayCount != Count, LogAndroid, Log, "nativeSetConfigRulesVariables KV pair array must have an even count.");
	int32 Index = 0;
	while (Index < Count)
	{
		auto javaKey = FJavaHelper::FStringFromLocalRef(jenv, (jstring)(jenv->GetObjectArrayElement(KeyValuePairs, Index++)));
		auto javaValue = FJavaHelper::FStringFromLocalRef(jenv, (jstring)(jenv->GetObjectArrayElement(KeyValuePairs, Index++)));

		FAndroidMisc::ConfigRulesVariables.Add(javaKey, javaValue);
	}

	{
		FPlatformMisc::BeginNamedEvent(FColor(0), TEXT("nativeSetConfigRulesVariables - ECR"));
		ON_SCOPE_EXIT
		{
			FPlatformMisc::EndNamedEvent();
		};

		jboolean isCopy;
		jbyte* Utf8bytes = Utf8Vars != nullptr ? jenv->GetByteArrayElements(Utf8Vars, &isCopy) : nullptr;
		static_assert(sizeof(UTF8CHAR) == sizeof(*Utf8bytes), "Size mismatch in cast");
		if (Utf8bytes != nullptr)
		{
			ON_SCOPE_EXIT
			{
				jenv->ReleaseByteArrayElements(Utf8Vars, Utf8bytes, JNI_ABORT);
			};

			TArrayView Utf8View(reinterpret_cast<UTF8CHAR*>(Utf8bytes), jenv->GetArrayLength(Utf8Vars));
			AndroidPlatformMisc::FEarlyConfigRuleReader& EarlyConfigRuleReader = AndroidPlatformMisc::GetConfigRuleReader(Utf8View);
			// if we were beaten to it then we still set the utf8 bytes version (early config data) if we somehow failed to set it.
			// this would be if an access before nativeSetConfigRulesVariables failed from time out or other socket error.
			// it should be very unlikely to be required, but is here as best effort to guarantee utf8 configrules variables are around
			// just as they were with the existing system.
			if (EarlyConfigRuleReader.SetBufferIfEmpty(Utf8View))
			{
				UE_LOGF(LogAndroid, Log, "nativeSetConfigRulesVariables set EarlyConfigRuleReader variables");
			}
			else
			{
				UE_LOGF(LogAndroid, Log, "nativeSetConfigRulesVariables ignored EarlyConfigRuleReader variables, already set.");
			}
		}
		if (jenv->ExceptionCheck())
		{
			UE_LOGF(LogAndroid, Warning, "nativeSetConfigRulesVariables: exception when fetching configrule bytes.");
			jenv->ExceptionClear();
		}
	}

	bConfigRulesReady = true;
}

#endif
extern bool AndroidThunkCpp_HasMetaDataKey(const FString& Key);

static bool bDetectedDebugger = false;

#if USE_ANDROID_JNI
void JNICALL UE::Jni::FGameActivity::nativeSetAndroidStartupState(JNIEnv* jenv, jobject thiz, jboolean bDebuggerAttached)
{
	// if Java debugger attached, mark detected (but don't lose previous trigger state)
	if (bDebuggerAttached)
	{
		bDetectedDebugger = true;
	}
}
#endif
bool FAndroidMisc::UseNewWindowBehavior()
{
#if USE_ANDROID_STANDALONE
	return true;
#endif
	static bool bUseNewWindowBehavior = FPlatformMisc::GetConfigRulesVariableAsUtf8("EnableNewBackgroundBehavior").Equals("true");
	return bUseNewWindowBehavior;
}

#if !UE_BUILD_SHIPPING
bool FAndroidMisc::IsDebuggerPresent()
{
	if (GIgnoreDebugger)
	{
		return false;
	}

	if (bDetectedDebugger)
	{
		return true;
	}

	// If a process is tracing this one then TracerPid in /proc/self/status will
	// be the id of the tracing process. Use SignalHandler safe functions

	int StatusFile = open("/proc/self/status", O_RDONLY);
	if (StatusFile == -1)
	{
		// Failed - unknown debugger status.
		return false;
	}

	char Buffer[256];
	ssize_t Length = read(StatusFile, Buffer, sizeof(Buffer));

	bool bDebugging = false;
	const char* TracerString = "TracerPid:\t";
	const ssize_t LenTracerString = strlen(TracerString);
	int i = 0;

	while ((Length - i) > LenTracerString)
	{
		// TracerPid is found
		if (strncmp(&Buffer[i], TracerString, LenTracerString) == 0)
		{
			// 0 if no process is tracing.
			bDebugging = Buffer[i + LenTracerString] != '0';
			break;
		}
		++i;
	}

	close(StatusFile);

	// remember if we detected debugger so we can skip check next time
	if (bDebugging)
	{
		bDetectedDebugger = true;
	}

	return bDebugging;
}
#endif

#if STATS || ENABLE_STATNAMEDEVENTS

static void WriteTraceMarkerEvent(const ANSICHAR* Text, int32 TraceMarkerFileDescriptor)
{
	ATrace_beginSection(Text);
}

void FAndroidMisc::BeginNamedEvent(const struct FColor& Color, const TCHAR* Text)
{
	FGenericPlatformMisc::BeginNamedEvent(Color, Text);
 	if (!ATrace_isEnabled())
	{
		return;
	}

	const int MAX_TRACE_MESSAGE_LENGTH = 256;
	WriteTraceMarkerEvent(StringCast<ANSICHAR, MAX_TRACE_MESSAGE_LENGTH>(Text).Get(), TraceMarkerFileDescriptor);
}

void FAndroidMisc::BeginNamedEvent(const struct FColor& Color, const ANSICHAR* Text)
{
	FGenericPlatformMisc::BeginNamedEvent(Color, Text);
	if (!ATrace_isEnabled() )
	{
		return;
	}

	WriteTraceMarkerEvent(Text, TraceMarkerFileDescriptor);
}

void FAndroidMisc::EndNamedEvent()
{
	FGenericPlatformMisc::EndNamedEvent();
 	if (!ATrace_isEnabled() )
	{
		return;
	}

	ATrace_endSection();
}

#endif // STATS || ENABLE_STATNAMEDEVENTS

int FAndroidMisc::GetVolumeState(double* OutTimeOfChangeInSec)
{
	int v;
	ReceiversLock.Lock();
	v = CurrentVolume.Volume;
	if (OutTimeOfChangeInSec)
	{
		*OutTimeOfChangeInSec = CurrentVolume.TimeOfChange;
	}
	ReceiversLock.Unlock();
	return v;
}

int32 FAndroidMisc::GetDeviceVolume()
{
	//FAndroidMisc::GetVolumeState returns 0-15, scale to 0-100
	int32 BaseVolume = FAndroidMisc::GetVolumeState();
	int32 ScaledVolume = (BaseVolume * 100) / 15;
	return ScaledVolume;
}

#if USE_ANDROID_FILE
const TCHAR* FAndroidMisc::GamePersistentDownloadDir()
{
	extern FString GExternalFilePath;
	return *GExternalFilePath;
}

FString FAndroidMisc::GetLoginId()
{
	static FString LoginId;

	// Return already loaded or generated Id
	if (!LoginId.IsEmpty())
	{
		return LoginId;
	}

	// Check for existing identifier file
	extern FString GInternalFilePath;
	extern FString GExternalFilePath;
	FString InternalLoginIdFilename = GInternalFilePath / TEXT("login-identifier.txt");
	if (FPaths::FileExists(InternalLoginIdFilename))
	{
		if (FFileHelper::LoadFileToString(LoginId, *InternalLoginIdFilename))
		{
			return LoginId;
		}
	}
	FString LoginIdFilename = GExternalFilePath / TEXT("login-identifier.txt");
	if (FPaths::FileExists(LoginIdFilename))
	{
		if (FFileHelper::LoadFileToString(LoginId, *LoginIdFilename))
		{
			FFileHelper::SaveStringToFile(LoginId, *InternalLoginIdFilename);
			return LoginId;
		}
	}

	// Generate a new one and write to file
	FGuid DeviceGuid;
	FPlatformMisc::CreateGuid(DeviceGuid);
	LoginId = DeviceGuid.ToString();
	FFileHelper::SaveStringToFile(LoginId, *InternalLoginIdFilename);

	return LoginId;
}
#endif

#if USE_ANDROID_JNI
FString FAndroidMisc::GetDeviceId()
{
#if GET_DEVICE_ID_UNAVAILABLE
	return FString();
#else
	extern FString AndroidThunkCpp_GetAndroidId();
	static FString DeviceId = AndroidThunkCpp_GetAndroidId();

	// note: this can be empty or NOT unique depending on the OEM implementation!
	return DeviceId;
#endif
}

FString FAndroidMisc::GetUniqueAdvertisingId()
{
	extern FString AndroidThunkCpp_GetAdvertisingId();
	static FString AdvertisingId = AndroidThunkCpp_GetAdvertisingId();

	// note: this can be empty if Google Play not installed, or user is blocking it!
	return AdvertisingId;
}
#endif

FAndroidMisc::FBatteryState FAndroidMisc::GetBatteryState()
{
	FBatteryState CurState;
	ReceiversLock.Lock();
	CurState = CurrentBatteryState;
	ReceiversLock.Unlock();
	return CurState;
}

int FAndroidMisc::GetBatteryLevel()
{
	FBatteryState BatteryState = GetBatteryState();
	return BatteryState.Level;
}

bool FAndroidMisc::IsRunningOnBattery()
{
	FBatteryState BatteryState = GetBatteryState();
	return BatteryState.State == BATTERY_STATE_DISCHARGING;
}

bool FAndroidMisc::IsInLowPowerMode()
{
	FBatteryState BatteryState = GetBatteryState();
	return BatteryState.Level <= GAndroidLowPowerBatteryThreshold;
}

float FAndroidMisc::GetDeviceTemperatureLevel()
{
	return FAndroidMisc::GetDeviceTemperature();
}

float FAndroidMisc::GetDeviceTemperature()
{
	return GetBatteryState().Temperature;
}

EDeviceThermalState FAndroidMisc::GetDeviceThermalState()
{
	return FAndroidStats::GetThermalStatus();
}

bool FAndroidMisc::AreHeadPhonesPluggedIn()
{
	return HeadPhonesArePluggedIn;
}

#define ANDROIDTHUNK_CONNECTION_TYPE_NONE 0
#define ANDROIDTHUNK_CONNECTION_TYPE_AIRPLANEMODE 1
#define ANDROIDTHUNK_CONNECTION_TYPE_ETHERNET 2
#define ANDROIDTHUNK_CONNECTION_TYPE_CELL 3
#define ANDROIDTHUNK_CONNECTION_TYPE_WIFI 4
#define ANDROIDTHUNK_CONNECTION_TYPE_WIMAX 5
#define ANDROIDTHUNK_CONNECTION_TYPE_BLUETOOTH 6

static bool bLastConnectionTypeValid = false;
static ENetworkConnectionType LastNetworkConnectionType = ENetworkConnectionType::None;

static ENetworkConnectionType PrivateGetNetworkConnectionType()
{
#if USE_ANDROID_JNI
	extern int32 AndroidThunkCpp_GetNetworkConnectionType();

	switch (AndroidThunkCpp_GetNetworkConnectionType())
	{
		case ANDROIDTHUNK_CONNECTION_TYPE_NONE:				return ENetworkConnectionType::None;
		case ANDROIDTHUNK_CONNECTION_TYPE_AIRPLANEMODE:		return ENetworkConnectionType::AirplaneMode;
		case ANDROIDTHUNK_CONNECTION_TYPE_ETHERNET:			return ENetworkConnectionType::Ethernet;
		case ANDROIDTHUNK_CONNECTION_TYPE_CELL:				return ENetworkConnectionType::Cell;
		case ANDROIDTHUNK_CONNECTION_TYPE_WIFI:				return ENetworkConnectionType::WiFi;
		case ANDROIDTHUNK_CONNECTION_TYPE_WIMAX:			return ENetworkConnectionType::WiMAX;
		case ANDROIDTHUNK_CONNECTION_TYPE_BLUETOOTH:		return ENetworkConnectionType::Bluetooth;
	}
#endif
	return ENetworkConnectionType::Unknown;
}

ENetworkConnectionType FAndroidMisc::GetNetworkConnectionType()
{
	if (!bLastConnectionTypeValid)
	{
		LastNetworkConnectionType = PrivateGetNetworkConnectionType();
		bLastConnectionTypeValid = true;
	}
	return LastNetworkConnectionType;
}

#if USE_ANDROID_JNI
bool FAndroidMisc::HasActiveWiFiConnection()
{
	ENetworkConnectionType ConnectionType = GetNetworkConnectionType();
	return (ConnectionType == ENetworkConnectionType::WiFi ||
			ConnectionType == ENetworkConnectionType::WiMAX);
}

void JNICALL UE::Jni::FGameActivity::nativeNetworkChanged(JNIEnv* jenv, jobject thiz)
{
	LastNetworkConnectionType = PrivateGetNetworkConnectionType();
	bLastConnectionTypeValid = true;
	
	if (FTaskGraphInterface::IsRunning())
	{
		FFunctionGraphTask::CreateAndDispatchWhenReady([]()
		{
			const ENetworkConnectionType ConnectionType = FAndroidMisc::GetNetworkConnectionType();
			FCoreDelegates::OnNetworkConnectionChanged.Broadcast(ConnectionType);
			FPlatformMisc::SetNetworkConnectionStatus(FPlatformMisc::NetworkConnectionTypeToStatus(ConnectionType));
		}, TStatId(), NULL, ENamedThreads::GameThread);
	}
	else
	{
		// Task graph not running yet (early startup). Set the status synchronously so GetNetworkConnectionStatus() stays in sync with GetNetworkConnectionType().
		FPlatformMisc::SetNetworkConnectionStatus(FPlatformMisc::NetworkConnectionTypeToStatus(LastNetworkConnectionType));
	}
}

#endif
static FAndroidMisc::ReInitWindowCallbackType OnReInitWindowCallback;

FAndroidMisc::ReInitWindowCallbackType FAndroidMisc::GetOnReInitWindowCallback()
{
	return OnReInitWindowCallback;
}

void FAndroidMisc::SetOnReInitWindowCallback(FAndroidMisc::ReInitWindowCallbackType InOnReInitWindowCallback)
{
	OnReInitWindowCallback = InOnReInitWindowCallback;
}

static FAndroidMisc::ReleaseWindowCallbackType OnReleaseWindowCallback;

FAndroidMisc::ReleaseWindowCallbackType FAndroidMisc::GetOnReleaseWindowCallback()
{
	return OnReleaseWindowCallback;
}

void FAndroidMisc::SetOnReleaseWindowCallback(FAndroidMisc::ReleaseWindowCallbackType InOnReleaseWindowCallback)
{
	OnReleaseWindowCallback = InOnReleaseWindowCallback;
}

static FAndroidMisc::OnPauseCallBackType OnPauseCallback;

FAndroidMisc::OnPauseCallBackType FAndroidMisc::GetOnPauseCallback()
{
	return OnPauseCallback;
}

void FAndroidMisc::SetOnPauseCallback(FAndroidMisc::OnPauseCallBackType InOnPauseCallback)
{
	OnPauseCallback = InOnPauseCallback;
}

FString FAndroidMisc::GetCPUVendor()
{
	return DeviceMake;
}

FString FAndroidMisc::GetCPUBrand()
{
	return DeviceModel;
}

FString FAndroidMisc::GetCPUChipset()
{
	static const FString *Chipset = FAndroidMisc::GetConfigRulesVariable(TEXT("hardware"));
	return (Chipset == NULL) ? FGenericPlatformMisc::GetCPUChipset() : *Chipset;
}

FString FAndroidMisc::GetPrimaryGPUBrand()
{
	return FAndroidMisc::GetGPUFamily();
}

void FAndroidMisc::GetOSVersions(FString& out_OSVersionLabel, FString& out_OSSubVersionLabel)
{
	out_OSVersionLabel = TEXT("Android");
	out_OSSubVersionLabel = AndroidVersion;
}

FString FAndroidMisc::GetOSVersion()
{
	return AndroidVersion;
}

bool FAndroidMisc::GetDiskTotalAndFreeSpace(const FString& InPath, uint64& TotalNumberOfBytes, uint64& NumberOfFreeBytes)
{
#if USE_ANDROID_FILE
	extern FString GExternalFilePath;
	struct statfs FSStat = { 0 };
	FTCHARToUTF8 Converter(*GExternalFilePath);
	int Err = statfs((ANSICHAR*)Converter.Get(), &FSStat);

	if (Err == 0)
	{
		TotalNumberOfBytes = FSStat.f_blocks * FSStat.f_bsize;
		NumberOfFreeBytes = FSStat.f_bavail * FSStat.f_bsize;
	}
	else
	{
		int ErrNo = errno;
		UE_LOGF(LogAndroid, Warning, "Unable to statfs('%ls'): errno=%d (%ls)", *GExternalFilePath, ErrNo, UTF8_TO_TCHAR(strerror(ErrNo)));
	}

	return (Err == 0);
#else
	return false;
#endif
}

uint32 FAndroidMisc::GetCoreFrequency(int32 CoreIndex, ECoreFrequencyProperty CoreFrequencyProperty)
{
	uint32 ReturnFrequency = 0;
	char QueryFile[256];
	const char* FreqProperty = nullptr;
	static const char* CurrentFrequencyString = "scaling_cur_freq";
	static const char* MaxFrequencyString = "cpuinfo_max_freq";
	static const char* MinFrequencyString = "cpuinfo_min_freq";
	switch (CoreFrequencyProperty)
	{
		case ECoreFrequencyProperty::MaxFrequency:
			FreqProperty = MaxFrequencyString;
			break;
		case ECoreFrequencyProperty::MinFrequency:
			FreqProperty = MinFrequencyString;
			break;
		default:
		case ECoreFrequencyProperty::CurrentFrequency:
			FreqProperty = CurrentFrequencyString;
			break;
	}
	sprintf(QueryFile, "/sys/devices/system/cpu/cpu%d/cpufreq/%s", CoreIndex, FreqProperty);

	if (FILE* CoreFreqStateFile = fopen(QueryFile, "r"))
	{
		char CurrCoreFreq[32] = { 0 };
		if( fgets(CurrCoreFreq, UE_ARRAY_COUNT(CurrCoreFreq), CoreFreqStateFile) != nullptr)
		{
			ReturnFrequency = atol(CurrCoreFreq);
		}
		fclose(CoreFreqStateFile);
	}
	return ReturnFrequency;
}

float FAndroidMisc::GetCPUTemperature()
{
	float Temp = 0.0f;
	if (*AndroidCpuThermalSensorFileBuf == 0)
	{
		return Temp;
	}

	if (FILE* Thermals = fopen(AndroidCpuThermalSensorFileBuf, "r"))
	{
		char Buf[256];
		if (fgets(Buf, UE_ARRAY_COUNT(Buf), Thermals))
		{
			// sensor temp file can contain whitespace symbols at the end of the line, count length only for digit symbols
			char* p = Buf;
			uint32 Len = 0;
			while (isdigit(*p))
			{
				++Len;
				++p;
			}

			// Temperature is reported by different sensors in different ways, some report it as XXX, some - as XXXXX. Reduce it to standard XX.X
			const uint32 StandardLen = 2;
			const float Divider = pow(10.0f, (float)(Len - StandardLen));
			Temp = (float)atol(Buf) / Divider;
		}
		fclose(Thermals);
	}

	return Temp;
}

bool FAndroidMisc::Expand16BitIndicesTo32BitOnLoad()
{
	return  (CVarMaliMidgardIndexingBug.GetValueOnAnyThread() > 0);
}

int FAndroidMisc::GetMobilePropagateAlphaSetting()
{
	extern int GAndroidPropagateAlpha;
	return GAndroidPropagateAlpha;
}

TArray<int32> FAndroidMisc::GetSupportedNativeDisplayRefreshRates()
{
	TArray<int32> Result;
#if USE_ANDROID_JNI && !USE_ANDROID_STANDALONE
	extern TArray<int32> AndroidThunkCpp_GetSupportedNativeDisplayRefreshRates();
	Result = AndroidThunkCpp_GetSupportedNativeDisplayRefreshRates();
#else
	Result.Add(60);
#endif
	return Result;
}

bool FAndroidMisc::SetNativeDisplayRefreshRate(int32 RefreshRate)
{
#if USE_ANDROID_JNI && !USE_ANDROID_STANDALONE
	extern bool AndroidThunkCpp_SetNativeDisplayRefreshRate(int32 RefreshRate);
	return AndroidThunkCpp_SetNativeDisplayRefreshRate(RefreshRate);
#else
	return RefreshRate == 60;
#endif
}

int32 FAndroidMisc::GetNativeDisplayRefreshRate()
{
#if USE_ANDROID_JNI && !USE_ANDROID_STANDALONE
	extern int32 AndroidThunkCpp_GetNativeDisplayRefreshRate();
	return AndroidThunkCpp_GetNativeDisplayRefreshRate();
#else
	return 60;
#endif

}

FORCEINLINE bool ValueOutsideThreshold(float Value, float BaseLine, float Threshold)
{
	return Value > BaseLine * (1.0f + Threshold)
		|| Value < BaseLine * (1.0f - Threshold);
}

void (*GMemoryWarningHandler)(const FGenericMemoryWarningContext& Context) = NULL;

void FAndroidMisc::SetMemoryWarningHandler(void (*InHandler)(const FGenericMemoryWarningContext& Context))
{
	check(IsInGameThread());
	GMemoryWarningHandler = InHandler;
}

bool FAndroidMisc::HasMemoryWarningHandler()
{
	check(IsInGameThread());
	return GMemoryWarningHandler != nullptr;
}

bool FAndroidMisc::SupportsBackbufferSampling()
{
	static int32 CachedAndroidOpenGLSupportsBackbufferSampling = -1;
	
	if (CachedAndroidOpenGLSupportsBackbufferSampling == -1)
	{
		bool bAndroidOpenGLSupportsBackbufferSampling = false;
		GConfig->GetBool(TEXT("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings"), TEXT("bAndroidOpenGLSupportsBackbufferSampling"), bAndroidOpenGLSupportsBackbufferSampling, GEngineIni);

		CachedAndroidOpenGLSupportsBackbufferSampling = (bAndroidOpenGLSupportsBackbufferSampling || FAndroidMisc::ShouldUseVulkan()) ? 1 : 0;
	}

	return CachedAndroidOpenGLSupportsBackbufferSampling == 1;
}

void FAndroidMisc::NonReentrantRequestExit()
{
#if UE_SET_REQUEST_EXIT_ON_TICK_ONLY
	// Cheating here to grab access to this. This function should only be used in extreme cases in which non-reentrant functions are needed (ie. crash handling/signal handler)
	extern bool GShouldRequestExit;
	GShouldRequestExit = true;
#else
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	GIsRequestingExit = true;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // UE_SET_REQUEST_EXIT_ON_TICK_ONLY
}

void FAndroidMisc::RegisterThreadName(const char* Name, uint32 ThreadId)
{
	FScopeLock Lock(&AndroidThreadNamesLock);
	if (!AndroidThreadNames.Contains(ThreadId))
	{
		AndroidThreadNames.Add(ThreadId, Name);
	}
}

const char* FAndroidMisc::GetThreadName(uint32 ThreadId)
{
	FScopeLock Lock(&AndroidThreadNamesLock);
	const char** ThreadName = AndroidThreadNames.Find(ThreadId);
	return ThreadName ? *ThreadName : nullptr;
}

void FAndroidMisc::SetCellularPreference(int32 Value)
{
#if USE_ANDROID_JNI
	AndroidThunkCpp_SetSharedPreferenceInt(TEXT("CellularNetworkPreferences"), TEXT("AllowCellular"), Value);
#endif // USE_ANDROID_JNI
}

int32 FAndroidMisc::GetCellularPreference()
{
	int32 Result = 0;
#if USE_ANDROID_JNI
	Result = AndroidThunkCpp_GetSharedPreferenceInt(TEXT("CellularNetworkPreferences"), TEXT("AllowCellular"), Result);
#endif // USE_ANDROID_JNI
	return Result;
}

void FAndroidMisc::SetAllowedDeviceOrientation(EDeviceScreenOrientation NewAllowedDeviceOrientation)
{
	AllowedDeviceOrientation = NewAllowedDeviceOrientation;

#if USE_ANDROID_JNI && !USE_ANDROID_STANDALONE
	AndroidThunkCpp_SetOrientation(GetAndroidScreenOrientation(NewAllowedDeviceOrientation));
#endif // USE_ANDROID_JNI
}

#if USE_ANDROID_JNI && !USE_ANDROID_STANDALONE
int32 FAndroidMisc::GetAndroidScreenOrientation(EDeviceScreenOrientation ScreenOrientation)
{
	EAndroidScreenOrientation AndroidScreenOrientation = EAndroidScreenOrientation::SCREEN_ORIENTATION_UNSPECIFIED;
	switch (ScreenOrientation)
	{
	case EDeviceScreenOrientation::Unknown:
		AndroidScreenOrientation = EAndroidScreenOrientation::SCREEN_ORIENTATION_UNSPECIFIED;
		break;
	case EDeviceScreenOrientation::Portrait:
		AndroidScreenOrientation = EAndroidScreenOrientation::SCREEN_ORIENTATION_PORTRAIT;
		break;
	case EDeviceScreenOrientation::PortraitUpsideDown:
		AndroidScreenOrientation = EAndroidScreenOrientation::SCREEN_ORIENTATION_REVERSE_PORTRAIT;
		break;
	case EDeviceScreenOrientation::LandscapeLeft:
		AndroidScreenOrientation = EAndroidScreenOrientation::SCREEN_ORIENTATION_LANDSCAPE;
		break;
	case EDeviceScreenOrientation::LandscapeRight:
		AndroidScreenOrientation = EAndroidScreenOrientation::SCREEN_ORIENTATION_REVERSE_LANDSCAPE;
		break;
	case EDeviceScreenOrientation::FaceUp:
		AndroidScreenOrientation = EAndroidScreenOrientation::SCREEN_ORIENTATION_UNSPECIFIED;
		break;
	case EDeviceScreenOrientation::FaceDown:
		AndroidScreenOrientation = EAndroidScreenOrientation::SCREEN_ORIENTATION_UNSPECIFIED;
		break;
	case EDeviceScreenOrientation::PortraitSensor:
		AndroidScreenOrientation = EAndroidScreenOrientation::SCREEN_ORIENTATION_SENSOR_PORTRAIT;
		break;
	case EDeviceScreenOrientation::LandscapeSensor:
		AndroidScreenOrientation = EAndroidScreenOrientation::SCREEN_ORIENTATION_SENSOR_LANDSCAPE;
		break;
	case EDeviceScreenOrientation::FullSensor:
		AndroidScreenOrientation = EAndroidScreenOrientation::SCREEN_ORIENTATION_SENSOR;
		break;
	}

	return static_cast<int32>(AndroidScreenOrientation);
}
#endif // USE_ANDROID_JNI && !USE_ANDROID_STANDALONE

extern void AndroidThunkCpp_ShowConsoleWindow();
void FAndroidMisc::ShowConsoleWindow()
{
#if !UE_BUILD_SHIPPING && USE_ANDROID_JNI
	AndroidThunkCpp_ShowConsoleWindow();
#endif // !UE_BUILD_SHIPPING && USE_ANDROID_JNI
}

FDelegateHandle FAndroidMisc::AddNetworkListener(FCoreDelegates::FOnNetworkConnectionChanged::FDelegate&& InNewDelegate)
{
	// not really necessary since PlatformInit calls AddNetworkListener but doesn't hurt anything
	if (!FCoreDelegates::OnNetworkConnectionChanged.IsBound())
	{
#if USE_ANDROID_JNI
		extern void AndroidThunkJava_AddNetworkListener();
		AndroidThunkJava_AddNetworkListener();
#endif
	}

	return FCoreDelegates::OnNetworkConnectionChanged.Add(MoveTemp(InNewDelegate));
}

bool FAndroidMisc::RemoveNetworkListener(FDelegateHandle Handle)
{
	bool bSuccess = FCoreDelegates::OnNetworkConnectionChanged.Remove(Handle);

	// we don't really want to remove listener since we're using it for GetNetworkConnectionType
//	if (!FCoreDelegates::OnNetworkConnectionChanged.IsBound())
//	{
//#if USE_ANDROID_JNI
//		extern void AndroidThunkJava_RemoveNetworkListener();
//		AndroidThunkJava_RemoveNetworkListener();
//#endif
//	}

	return bSuccess;
}

float FAndroidMisc::GetVirtualKeyboardInputHeight()
{
#if USE_ANDROID_JNI
	extern float AndroidThunkCpp_GetMetaDataFloat(const FString& Key);
	return AndroidThunkCpp_GetMetaDataFloat(TEXT("unreal.input.virtualKeyboardInputHeight"));
#else
	return 0.0f;
#endif	
}

#if !UE_BUILD_SHIPPING
bool FAndroidMisc::IsConsoleOpen()
{
	return GDebugConsoleOpen;
}
#endif
