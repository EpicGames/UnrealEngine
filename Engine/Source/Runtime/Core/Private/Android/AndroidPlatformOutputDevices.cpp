// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AndroidOutputDevices.cpp: Android implementations of OutputDevices functions
=============================================================================*/

#include "Android/AndroidPlatformOutputDevices.h"

/*class FOutputDevice* FAndroidOutputDevices::GetLog()
{
#if UE_BUILD_SHIPPING
	return NULL;
#else
	// Always enable logging via ADB
	static FOutputDeviceAndroidDebug Singleton;
	return &Singleton;
	return NULL;
#endif
}*/
