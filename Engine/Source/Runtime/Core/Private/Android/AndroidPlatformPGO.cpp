// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreGlobals.h"
#include "Misc/Parse.h"
#include "Misc/CommandLine.h"

#if PLATFORM_COMPILER_OPTIMIZATION_PG_PROFILING
extern "C" void __llvm_profile_reset_counters(void);
extern "C" int __llvm_profile_write_file(void);
extern "C" void __llvm_profile_set_filename(char*);

extern FString AndroidRelativeToAbsolutePath(bool bUseInternalBasePath, FString RelPath);

static uint64 PGOFileCounter = 0;
FString PGO_GetOutputDirectory()
{
	FString PGOOutputDirectory;
	if (FParse::Value(FCommandLine::Get(), TEXT("pgoprofileoutput="), PGOOutputDirectory))
	{
		return PGOOutputDirectory;
	}

	FString DefaultPGODir = AndroidRelativeToAbsolutePath(false, FPaths::ProjectSavedDir());
	UE_LOGF(LogAndroid, Warning, "No PGO output destination path specifed, defaulting to %ls", *DefaultPGODir);
	return DefaultPGODir;
}

static void PGO_ResetCounters()
{
	// Reset all counters, essentially restarting the profile run.

	UE_LOGF(LogAndroid, Log, "Resetting PGO counters.");
	__llvm_profile_reset_counters();
}

void PGO_WriteFile()
{
	static const FString OutputDirectory = PGO_GetOutputDirectory();
	FString OutputFileName = OutputDirectory + FString::Printf(TEXT("/%d.profraw"), ++PGOFileCounter);

	UE_LOGF(LogAndroid, Log, "Writing out PGO results file: \"%ls\".", *OutputFileName);
	__llvm_profile_set_filename(TCHAR_TO_ANSI(*OutputFileName));

	if (__llvm_profile_write_file() != 0)
	{
		UE_LOGF(LogAndroid, Error, "Failed to write PGO output file.");
	}
	else
	{
		UE_LOGF(LogAndroid, Log, "PGO results file written successfully.");
	}

	// Reset counters after writing a file so we don't count the
	// profiling data twice if another file is written out.
	PGO_ResetCounters();
}
#endif //PLATFORM_COMPILER_OPTIMIZATION_PG_PROFILING