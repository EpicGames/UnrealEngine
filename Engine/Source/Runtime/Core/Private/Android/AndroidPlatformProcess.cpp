// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AndroidProcess.cpp: Android implementations of Process functions
=============================================================================*/

#include "Android/AndroidPlatformProcess.h"
#include "Android/AndroidJniGameActivity.h"
#include "Android/AndroidPlatform.h"
#include "Android/AndroidPlatformRunnableThread.h"
#include "Android/AndroidPlatformAffinity.h"
#include "Android/AndroidPlatformMisc.h"
#include "Async/TaskGraphInterfaces.h"

#include <sys/syscall.h>
#include <pthread.h>
#include <dlfcn.h>
#include <sys/resource.h>

#include "Misc/CoreDelegates.h"

// RTLD_NOLOAD not defined for all platforms before NDK15
#if PLATFORM_ANDROID_NDK_VERSION < 150000
	// not defined for NDK platform before 21
	#if PLATFORM_USED_NDK_VERSION_INTEGER < 21
		#define RTLD_NOLOAD   0x00004
	#endif
#endif

uint64 FAndroidAffinity::GameThreadMask = FPlatformAffinity::GetNoAffinityMask();
uint64 FAndroidAffinity::RenderingThreadMask = FPlatformAffinity::GetNoAffinityMask();

void* FAndroidPlatformProcess::GetDllHandle(const TCHAR* Filename)
{
	check(Filename);

	// Check if dylib is already loaded
	void* Handle = dlopen(TCHAR_TO_ANSI(Filename), RTLD_NOLOAD | RTLD_LAZY | RTLD_LOCAL);
	
	if (!Handle)
	{
		// Not loaded yet, so try to open it
		Handle = dlopen(TCHAR_TO_ANSI(Filename), RTLD_LAZY | RTLD_LOCAL);
	}
	if (!Handle)
	{
		UE_LOGF(LogAndroid, Warning, "dlopen failed: %ls", ANSI_TO_TCHAR(dlerror()));
	}
	return Handle;
}

void FAndroidPlatformProcess::FreeDllHandle(void* DllHandle)
{
	check(DllHandle);
	dlclose(DllHandle);
}

void* FAndroidPlatformProcess::GetDllExport(void* DllHandle, const TCHAR* ProcName)
{
	check(DllHandle);
	check(ProcName);
	return dlsym(DllHandle, TCHAR_TO_ANSI(ProcName));
}

const TCHAR* FAndroidPlatformProcess::ComputerName()
{
	static FString ComputerName;
	if (ComputerName.Len() == 0)
	{
		ComputerName = FAndroidMisc::GetDeviceModel();
	}

	return *ComputerName; 
}

void FAndroidPlatformProcess::SetThreadAffinityMask( uint64 InAffinityMask )
{
	/* 
	 * On Android we prefer not to touch the thread affinity at all unless the user has specifically requested to change it.
	 * The only way to override the default mask is to use the android.DefaultThreadAffinity console command set by ini file or device profile.
	 */
	if (FPlatformAffinity::GetNoAffinityMask() != InAffinityMask)
	{
		pid_t ThreadId = gettid();
		int AffinityMask = (int)InAffinityMask;
		syscall(__NR_sched_setaffinity, ThreadId, sizeof(AffinityMask), &AffinityMask);
	}
}

uint32 FAndroidPlatformProcess::GetCurrentProcessId()
{
	return getpid();
}

uint32 FAndroidPlatformProcess::GetCurrentCoreNumber()
{
	unsigned CPU;
	int Err = syscall(__NR_getcpu, &CPU, nullptr, nullptr);
	return (!Err) ? CPU : 0;
}

const TCHAR* FAndroidPlatformProcess::BaseDir()
{
	return TEXT("");
}

const TCHAR* FAndroidPlatformProcess::ExecutableName(bool bRemoveExtension)
{
#if USE_ANDROID_FILE
	extern FString GAndroidProjectName;
	return *GAndroidProjectName;
#else
	UE_LOGF(LogAndroid, Fatal, "A sub-platform that doesn't use USE_ANDROID_FILE must implement PlatformProcess::ExecutableName");
	return TEXT("");
#endif
}

FRunnableThread* FAndroidPlatformProcess::CreateRunnableThread()
{
	return new FRunnableThreadAndroid();
}

bool FAndroidPlatformProcess::CanLaunchURL(const TCHAR* URL)
{
	return URL != nullptr;
}

DECLARE_DELEGATE_OneParam(FAndroidLaunchURLDelegate, const FString&);

CORE_API FAndroidLaunchURLDelegate OnAndroidLaunchURL;

void FAndroidPlatformProcess::LaunchURL(const TCHAR* URL, const TCHAR* Parms, FString* Error)
{
	if (FCoreDelegates::ShouldLaunchUrl.IsBound() && !FCoreDelegates::ShouldLaunchUrl.Execute(URL))
	{
		if (Error)
		{
			*Error = TEXT("LaunchURL cancelled by delegate");
		}
		return;
	}

	check(URL);
	const FString URLWithParams = FString::Printf(TEXT("%s %s"), URL, Parms ? Parms : TEXT("")).TrimEnd();

	OnAndroidLaunchURL.ExecuteIfBound(URLWithParams);

	if (Error != nullptr)
	{
		*Error = TEXT("");
	}
}

FString FAndroidPlatformProcess::GetGameBundleId()
{
#if USE_ANDROID_JNI
	JNIEnv* JEnv = AndroidJavaEnv::GetJavaEnv();
	if (nullptr != JEnv)
	{
		jclass Class = AndroidJavaEnv::FindJavaClassGlobalRef(ANDROID_GAMEACTIVITY_BASE_CLASSPATH);
		if (nullptr != Class)
		{
			jmethodID getAppPackageNameMethodId = JEnv->GetStaticMethodID(Class, "getAppPackageName", "()Ljava/lang/String;");
			FString PackageName = FJavaHelper::FStringFromLocalRef(JEnv, (jstring)JEnv->CallStaticObjectMethod(Class, getAppPackageNameMethodId, nullptr));
			JEnv->DeleteGlobalRef(Class);
			return PackageName;
		}
	}
#endif
	return TEXT("");
}

void FAndroidPlatformProcess::SetThreadName(const TCHAR* ThreadName)
{
	pthread_setname_np(pthread_self(), TCHAR_TO_ANSI(ThreadName));
}

bool FAndroidPlatformProcess::GetPerFrameProcessorUsage(uint32 ProcessId, float& ProcessUsageFraction, float& IdleUsageFraction)
{
	(void)(ProcessId);
	FAndroidMisc::FCPUState& AndroidCPUState = FAndroidMisc::GetCPUState();
	ProcessUsageFraction = (float)AndroidCPUState.AverageUtilization;
	// Cheat for now by assuming idle time + utilization always equals 1.0f
	IdleUsageFraction = 1.0f - (float)AndroidCPUState.AverageUtilization;

	// We memset the structure zero, so if we have failed for whatever reason this is our sentinel value
	return AndroidCPUState.CoreCount != 0;
}

// Can be specified per device profile
// android.DefaultThreadAffinity GT 0x01 RT 0x02
TAutoConsoleVariable<FString> CVarAndroidDefaultThreadAffinity(
	TEXT("android.DefaultThreadAffinity"), 
	TEXT(""), 
	TEXT("Sets the thread affinity for Android platform. Pairs of args [GT|RT] [Hex affinity], ex: android.DefaultThreadAffinity GT 0x01 RT 0x02"));

static void AndroidSetAffinityOnThread()
{
	if (IsInActualRenderingThread()) // If RenderingThread is not started yet, affinity will applied at RT creation time 
	{
		FPlatformProcess::SetThreadAffinityMask(FPlatformAffinity::GetRenderingThreadMask());
	}
	else if (IsInGameThread())
	{
		FPlatformProcess::SetThreadAffinityMask(FPlatformAffinity::GetMainGameMask());
	}
}

static void ApplyDefaultThreadAffinity(IConsoleVariable* Var)
{
	FString AffinityCmd = CVarAndroidDefaultThreadAffinity.GetValueOnAnyThread();

	TArray<FString> Args;
	if (AffinityCmd.ParseIntoArrayWS(Args) > 0)
	{
		for (int32 Index = 0; Index + 1 < Args.Num(); Index += 2)
		{
			uint64 Aff = FParse::HexNumber(*Args[Index + 1]);
			if (!Aff)
			{
				UE_LOGF(LogAndroid, Display, "Parsed 0 for affinity, using 0xFFFFFFFFFFFFFFFF instead");
				Aff = 0xFFFFFFFFFFFFFFFF;
			}

			if (Args[Index] == TEXT("GT"))
			{
				FAndroidAffinity::GameThreadMask = Aff;
			}
			else if (Args[Index] == TEXT("RT"))
			{
				FAndroidAffinity::RenderingThreadMask = Aff;
			}
		}

		if (FTaskGraphInterface::IsRunning())
		{
			FSimpleDelegateGraphTask::CreateAndDispatchWhenReady(
				FSimpleDelegateGraphTask::FDelegate::CreateStatic(&AndroidSetAffinityOnThread),
				TStatId(), NULL, ENamedThreads::GetRenderThread());

			FSimpleDelegateGraphTask::CreateAndDispatchWhenReady(
				FSimpleDelegateGraphTask::FDelegate::CreateStatic(&AndroidSetAffinityOnThread),
				TStatId(), NULL, ENamedThreads::GameThread);
		}
		else
		{
			AndroidSetAffinityOnThread();
		}
	}
}

void AndroidSetupDefaultThreadAffinity()
{
	ApplyDefaultThreadAffinity(nullptr);

	// Watch for CVar update
	CVarAndroidDefaultThreadAffinity->SetOnChangedCallback(FConsoleVariableDelegate::CreateStatic(&ApplyDefaultThreadAffinity));
}

static bool EnableLittleCoreAffinity = false;
static int32 BigCoreMask = 0;
static int32 LittleCoreMask = 0;

#if USE_ANDROID_JNI
//This function is declared in the Java-defined class, GameActivity.java: "public native void nativeSetAffinityInfo(boolean bEnableAffinity, int bigCoreMask, int littleCoreMask);
void JNICALL UE::Jni::FGameActivity::nativeSetAffinityInfo(JNIEnv* jenv, jobject thiz, jboolean bEnableAffinity, jint bigCoreMask, jint littleCoreMask)
{
	EnableLittleCoreAffinity = bEnableAffinity;
	BigCoreMask = bigCoreMask;
	LittleCoreMask = littleCoreMask;
}

#endif
uint64 FAndroidAffinity::GetLittleCoreMask()
{
	static int Mask = 0;
	if (Mask == 0)
	{
		Mask = FGenericPlatformAffinity::GetNoAffinityMask();
		if (EnableLittleCoreAffinity)
		{
			Mask = LittleCoreMask;
			FPlatformMisc::LowLevelOutputDebugStringf(TEXT("LittleCore Affinity applying mask: 0x%0x"), Mask);
		}
	}
	return Mask;
}

static int AndroidThreadPriorityToNice(EThreadPriority NewPriority)
{
	static const int LOWEST = 19;
	static const int BACKGROUND = 10;
	static const int NORMAL = 0;
	static const int FOREGROUND = -2;
	static const int URGENT = -8;

	static const int ThreadPriorityToNiceValue[7] = {
		NORMAL	,	//	TPri_Normal
		NORMAL - 2,	//	TPri_AboveNormal
		NORMAL + 3,	//	TPri_BelowNormal
		NORMAL - 4,	//	TPri_Highest
		BACKGROUND,	//	TPri_Lowest
		NORMAL+2,	//	TPri_SlightlyBelowNormal
		URGENT,		//	TPri_TimeCritical
	};
	static_assert(UE_ARRAY_COUNT(ThreadPriorityToNiceValue) == EThreadPriority::TPri_Num, "This array is expected to be 1:1 mapping with EThreadPriority.");
	return ThreadPriorityToNiceValue[(int)NewPriority];
}

#if ANDROID_USE_NICE_VALUE_THREADPRIORITY
void FRunnableThreadAndroid::SetThreadPriority(pthread_t InThread, EThreadPriority NewPriority)
{
	check((int)NewPriority >= 0 && NewPriority < EThreadPriority::TPri_Num);
	NewPriority = (EThreadPriority)FMath::Clamp((int)NewPriority, 0, (int)(EThreadPriority::TPri_Num - 1));

	// Read the current policy
	int32 InitialPolicy;
	int32 NewPolicy = (NewPriority == EThreadPriority::TPri_Lowest) ? SCHED_BATCH : SCHED_NORMAL;
	struct sched_param Sched = { };
	pthread_getschedparam(InThread, &InitialPolicy, &Sched);

	// The task system can call this function with high frequency,
	// in cases where the OS consistently prevents the requested pri changes we must limit logging.
	const bool bCanLog = ErrorLogLimit >= 0;
	bool bErrorEncountered = false;

	if (InitialPolicy != NewPolicy && (sched_setscheduler(ThreadID, NewPolicy, &Sched)!=0) && bCanLog)
	{
		bErrorEncountered = true;
		UE_LOGF(LogHAL, Error, "Failed to set %ls thread scheduler, tid %d from %d to %d (errno %d)", *GetThreadName(), ThreadID, InitialPolicy, NewPolicy, errno);
	}

	int InitialNice = getpriority(PRIO_PROCESS, ThreadID);
	int NewNice = AndroidThreadPriorityToNice(NewPriority);
	if (InitialNice != NewNice && (setpriority(PRIO_PROCESS, ThreadID, NewNice) != 0) && bCanLog)
	{
		bErrorEncountered = true;
		UE_LOGF(LogHAL, Error, "Failed to set %ls thread priority, tid %d from %d to %d (errno %d)", *GetThreadName(), ThreadID, InitialNice, NewNice, errno);
	}

	if (bCanLog && bErrorEncountered)
	{
		ErrorLogLimit--;
	}
}
#endif
