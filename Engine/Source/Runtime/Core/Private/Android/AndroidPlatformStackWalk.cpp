// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AndroidPlatformStackWalk.cpp: Android implementations of stack walk functions
=============================================================================*/

#include "Android/AndroidPlatformStackWalk.h"
#include "HAL/PlatformMemory.h"
#include "Misc/CString.h"
#include <unwind.h>
#include <dlfcn.h>
#include <cxxabi.h>
#include <stdio.h>
#include <signal.h>
#include <setjmp.h>
#include <pthread.h>
#include <android/log.h>
#include "Android/AndroidSignals.h"

#include <android/log.h>

#include "Misc/OutputDevice.h"
#include "Logging/LogMacros.h"

#define HAS_LIBUNWIND PLATFORM_ANDROID_ARM64

#if HAS_LIBUNWIND
#define UNW_LOCAL_ONLY
#include "libunwind.h"
#endif
#if ANDROID_HAS_RTSIGNALS
#include <syscall.h>
#endif
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformProcess.h"
#include "Misc/ScopeExit.h"

// Some devices with Android 10 have XOM security feature and walking stack might crash
// We need to verify first that it's safe to read callstack in a call toInitStackWalking
// Otherwise stack walking will be disabled
static sigjmp_buf XomJmp;
static bool DisableStackBacktracing = true;
static bool StackWalkingInitialized = false;
static void XomSignalHandler(int Sig)
{
	siglongjmp(XomJmp, Sig);
}

bool FAndroidPlatformStackWalk::InitStackWalking()
{
	if (StackWalkingInitialized)
	{
		return true;
	}

#if HAS_LIBUNWIND
	// Without this stack walk might touch executable memory and ASan will terminate the app on that.
	// Xom protection presents the same issue and is enabled on android 10 devices
	// see https://source.android.com/devices/tech/debug/execute-only-memory
	// Please note that XOM is enabled on some devices even when building with TargetSDK < 29 (Oculus Quest 2)
	// and not enabled on some other devices at all, like Pixel 4
	sigset_t SignalSet;
	sigemptyset(&SignalSet);
	sigaddset(&SignalSet, SIGSEGV);

	struct sigaction SigAction;
	struct sigaction OldSigAction;
	sigset_t OldSignalSet;
	memset(&SigAction, 0, sizeof(SigAction));
	SigAction.sa_handler = XomSignalHandler;
	SigAction.sa_mask = SignalSet;

	sigprocmask(SIG_SETMASK, &SigAction.sa_mask, &OldSignalSet);
	sigaction(SIGSEGV, &SigAction, &OldSigAction);

	if (sigsetjmp(XomJmp, 1) == 0)
	{
		// first call to unw_backtrace will trigger some initial large allocations and if it happens during stack capturing on an exception we might get another out of memory exception
		const uint32 Depth = 16;
		void* Stack[Depth];
		unw_backtrace((void**)Stack, Depth);
		DisableStackBacktracing = false;
	}
	else
	{
		//unw_disable_signal_frame_test(1);
		__android_log_print(ANDROID_LOG_DEBUG, "UE", "XOM has been detected");
	}

	sigaction(SIGSEGV, &OldSigAction, nullptr);
	sigprocmask(SIG_SETMASK, &OldSignalSet, nullptr);

	StackWalkingInitialized = true;
#endif
	return true;
}

void FAndroidPlatformStackWalk::ProgramCounterToSymbolInfo(uint64 ProgramCounter, FProgramCounterSymbolInfo& out_SymbolInfo)
{
	Dl_info DylibInfo;
	int32 Result = dladdr((const void*)ProgramCounter, &DylibInfo);
	if (Result == 0)
	{
		return;
	}

	out_SymbolInfo.ProgramCounter = ProgramCounter;

	int32 Status = 0;
	ANSICHAR* DemangledName = NULL;

	// Increased the size of the demangle destination to reduce the chances that abi::__cxa_demangle will allocate
	// this causes the app to hang as malloc isn't signal handler safe. Ideally we wouldn't call this function in a handler.
	size_t DemangledNameLen = 8192;
	ANSICHAR DemangledNameBuffer[8192] = { 0 };
	DemangledName = abi::__cxa_demangle(DylibInfo.dli_sname, DemangledNameBuffer, &DemangledNameLen, &Status);

	if (DemangledName)
	{
		// C++ function
		FCStringAnsi::Sprintf(out_SymbolInfo.FunctionName, "%s ", DemangledName);
	}
	else if (DylibInfo.dli_sname)
	{
		// C function
		FCStringAnsi::Sprintf(out_SymbolInfo.FunctionName, "%s() ", DylibInfo.dli_sname);
	}
	else
	{
		// Unknown!
		FCStringAnsi::Sprintf(out_SymbolInfo.FunctionName, "[Unknown]() ");
	}

	// No line number available.
	// TODO open libUnreal.so from the apk and get the DWARF-2 data.
	FCStringAnsi::Strcat(out_SymbolInfo.Filename, "Unknown");
	out_SymbolInfo.LineNumber = 0;

	// Offset of the symbol in the module, eg offset into libUnreal.so needed for offline addr2line use.
	out_SymbolInfo.OffsetInModule = ProgramCounter - (uint64)DylibInfo.dli_fbase;

	// Write out Module information.
	ANSICHAR* DylibPath = (ANSICHAR*)DylibInfo.dli_fname;
	ANSICHAR* DylibName = FCStringAnsi::Strrchr(DylibPath, '/');
	if (DylibName)
	{
		DylibName += 1;
	}
	else
	{
		DylibName = DylibPath;
	}
	FCStringAnsi::Strcpy(out_SymbolInfo.ModuleName, DylibName);
}

namespace AndroidStackWalkHelpers
{
	uint64* BackTrace;
	uint32 MaxDepth;

	static _Unwind_Reason_Code BacktraceCallback(struct _Unwind_Context* Context, void* InDepthPtr)
	{
		uint32* DepthPtr = (uint32*)InDepthPtr;

		// stop if filled the buffer
		if (*DepthPtr >= MaxDepth)
		{
			return _Unwind_Reason_Code::_URC_END_OF_STACK;
		}

		uint64 ip = (uint64)_Unwind_GetIP(Context);
		if (ip)
		{
			BackTrace[*DepthPtr] = ip;
			(*DepthPtr)++;
		}
		return _Unwind_Reason_Code::_URC_NO_REASON;
	}
}

#if HAS_LIBUNWIND
// code based on unw_backtrace using signal context for the walk, note that this code was originally intended to walk the current stack.
// Since it walks a signal context it includes the first frame.
static int backtrace_signal(void* sigcontext, void **buffer, int size)
{
	unw_cursor_t cursor;
	unw_word_t ip;
	int n = 0;

	if (unw_init_local2(&cursor, (unw_context_t *)sigcontext, 1) < 0)
	{
		return 0;
	}

	do
	{
		if (n >= size)
		{
			return n;
		}

		if (unw_get_reg(&cursor, UNW_REG_IP, &ip) < 0)
		{
			return n;
		}
		buffer[n++] = (void *)(uintptr_t)ip;
	} while (unw_step(&cursor) > 0);

	return n;
}
#endif

extern int32 unwind_backtrace_signal(void* sigcontext, uint64* Backtrace, int32 MaxDepth);

uint32 FAndroidPlatformStackWalk::CaptureStackBackTrace(uint64* BackTrace, uint32 MaxDepth, void* Context)
{
	if (DisableStackBacktracing)
	{
		return 0;
	}

	// Make sure we have place to store the information
	if (BackTrace == NULL || MaxDepth == 0)
	{
		return 0;
	}

	// zero results
	FPlatformMemory::Memzero(BackTrace, MaxDepth * sizeof(uint64));
	
	auto DefaultUnwindBacktrace = [BackTrace, MaxDepth]()
	{
		AndroidStackWalkHelpers::BackTrace = BackTrace;
		AndroidStackWalkHelpers::MaxDepth = MaxDepth;
		uint32 Depth = 0;
		_Unwind_Backtrace(AndroidStackWalkHelpers::BacktraceCallback, &Depth);
		return Depth;
	};

#if PLATFORM_ANDROID_ARM
	if (Context)
	{
		// Android signal handlers always catch signals before user handlers and passes it down to user later
		// _Unwind_Backtrace does not use signal context and will produce wrong callstack in this case
		// We use code from libcorkscrew to unwind backtrace using actual signal context
		// Code taken from https://android.googlesource.com/platform/system/core/+/jb-dev/libcorkscrew/arch-arm/backtrace-arm.c
		return unwind_backtrace_signal(Context, BackTrace, MaxDepth);
	}
	else
	{
		return DefaultUnwindBacktrace();
	}
#elif HAS_LIBUNWIND
	if (Context)
	{
		// Android signal handlers always catch signals before user handlers and passes it down to user later
		// unw_backtrace does not use signal context and will produce wrong callstack in this case
		// We use code from libunwind to unwind backtrace using actual signal context
		return backtrace_signal(Context, (void**)BackTrace, MaxDepth);
	}
	else
	{
		return unw_backtrace((void**)BackTrace, MaxDepth);
	}
#else
	return DefaultUnwindBacktrace();
#endif 

}

uint32 FAndroidPlatformStackWalk::CaptureStackBackTraceViaFramePointerWalking(uint64* BackTrace, uint32 MaxDepth)
{
#if PLATFORM_ANDROID_ARM64 || PLATFORM_ANDROID_X64
	uint64 StackTopPtr = 0;

	// pthread_getattr_np/pthread_attr_getstack are slow for main thread, so let's cache the stack top 
	const bool bIsMainThread = gettid() == getpid();
	static uint64 StackTopMainThread = 0;
	if (bIsMainThread && StackTopMainThread != 0)
	{
		StackTopPtr = StackTopMainThread;
	}
	else
	{
		pthread_attr_t ThreadAttr;
		pthread_getattr_np(pthread_self(), &ThreadAttr);

		void* StackBase;
		size_t StackSize;
		pthread_attr_getstack(&ThreadAttr, &StackBase, &StackSize);

		StackTopPtr = (uint64)StackBase + StackSize;

		if (bIsMainThread)
		{
			StackTopMainThread = StackTopPtr;
		}
	}

	struct StackFrame
	{
		StackFrame* NextFrame;
		UPTRINT ReturnPtr;

		inline UPTRINT GetReturnPtr()
		{
#if PLATFORM_ANDROID_ARM64
			register uintptr_t Ptr = ReturnPtr;
			asm("xpaclri" : "+r"(Ptr)); // this instruction is mapped to NOP on pre-PAC architectures
			return Ptr;
#else
			return ReturnPtr;
#endif
		}
	};

	StackFrame* FrameEnd = (StackFrame*)StackTopPtr;
	StackFrame* FrameStart = (StackFrame*)__builtin_frame_address(0);

	uint32 NumStackFrames = 0;

	for (StackFrame* CurrentFrame = FrameStart;
		NumStackFrames < MaxDepth &&
		CurrentFrame->NextFrame > FrameStart &&
		CurrentFrame->NextFrame <= FrameEnd &&
		((UPTRINT)CurrentFrame->NextFrame & (sizeof(StackFrame) - 1)) == 0 && // stop at unaligned frame
		CurrentFrame->GetReturnPtr() != 0; // stop if function ptr is 0
		CurrentFrame = CurrentFrame->NextFrame, NumStackFrames++)
	{
		BackTrace[NumStackFrames] = CurrentFrame->GetReturnPtr();
	}

	return NumStackFrames;
#else
	return 0;
#endif
}

bool FAndroidPlatformStackWalk::SymbolInfoToHumanReadableString(const FProgramCounterSymbolInfo& SymbolInfo, ANSICHAR* HumanReadableString, SIZE_T HumanReadableStringSize)
{
	const int32 MAX_TEMP_SPRINTF = 256;

	//
	// Callstack lines should be written in this standard format. These are parsed by tools so it is important that
	// extra elements are not inserted!
	//
	//	0xaddress module!func [file]
	// 
	// E.g. 0x045C8D01 OrionClient.self(0x00009034)!UEngine::PerformError() [D:\Epic\Orion\Engine\Source\Runtime\Engine\Private\UnrealEngine.cpp:6481]
	//
	// Module may be omitted, everything else should be present, or substituted with a string that conforms to the expected type
	//
	// E.g 0x00000000 (0x00000000) UnknownFunction []
	//
	// 
	if (HumanReadableString && HumanReadableStringSize > 0)
	{
		ANSICHAR StackLine[MAX_SPRINTF] = { 0 };

		// Strip module path.
		const ANSICHAR* Pos0 = FCStringAnsi::Strrchr(SymbolInfo.ModuleName, '\\');
		const ANSICHAR* Pos1 = FCStringAnsi::Strrchr(SymbolInfo.ModuleName, '/');
		const UPTRINT RealPos = FMath::Max((UPTRINT)Pos0, (UPTRINT)Pos1);
		const ANSICHAR* StrippedModuleName = RealPos > 0 ? (const ANSICHAR*)(RealPos + 1) : SymbolInfo.ModuleName;

		// Start with address
		ANSICHAR PCAddress[MAX_TEMP_SPRINTF] = { 0 };
		FCStringAnsi::Snprintf(PCAddress, MAX_TEMP_SPRINTF, "0x%016llX ", SymbolInfo.ProgramCounter);
		FCStringAnsi::StrncatTruncateDest(StackLine, MAX_SPRINTF, PCAddress);

		// Module if it's present
		const bool bHasValidModuleName = FCStringAnsi::Strlen(StrippedModuleName) > 0;
		if (bHasValidModuleName)
		{
			FCStringAnsi::StrncatTruncateDest(StackLine, MAX_SPRINTF, StrippedModuleName);
		}

		FCStringAnsi::Snprintf(PCAddress, MAX_TEMP_SPRINTF, "(0x%016llX)", SymbolInfo.OffsetInModule);
		FCStringAnsi::StrncatTruncateDest(StackLine, MAX_SPRINTF, PCAddress);
		FCStringAnsi::StrncatTruncateDest(StackLine, MAX_SPRINTF, "!");

		// Function if it's available, unknown if it's not
		const bool bHasValidFunctionName = FCStringAnsi::Strlen(SymbolInfo.FunctionName) > 0;
		if (bHasValidFunctionName)
		{
			FCStringAnsi::StrncatTruncateDest(StackLine, MAX_SPRINTF, SymbolInfo.FunctionName);
		}
		else
		{
			FCStringAnsi::StrncatTruncateDest(StackLine, MAX_SPRINTF, "UnknownFunction");
		}

		// file info
		const bool bHasValidFilename = FCStringAnsi::Strlen(SymbolInfo.Filename) > 0 && SymbolInfo.LineNumber > 0;
		if (bHasValidFilename)
		{
			ANSICHAR FilenameAndLineNumber[MAX_TEMP_SPRINTF] = { 0 };
			FCStringAnsi::Snprintf(FilenameAndLineNumber, MAX_TEMP_SPRINTF, " [%s:%i]", SymbolInfo.Filename, SymbolInfo.LineNumber);
			FCStringAnsi::StrncatTruncateDest(StackLine, MAX_SPRINTF, FilenameAndLineNumber);
		}
		else
		{
			FCStringAnsi::StrncatTruncateDest(StackLine, MAX_SPRINTF, " []");
		}

		// Append the stack line.
		FCStringAnsi::StrncatTruncateDest(HumanReadableString, HumanReadableStringSize, StackLine);

		// Return true, if we have a valid function name.
		return bHasValidFunctionName;
	}
	return false;
}


static float GThreadCallStackRequestMaxWait = 0.5f;
static FAutoConsoleVariableRef CVarAndroidPlatformThreadCallStackRequestMaxWait(
	TEXT("AndroidPlatformThreadStackWalk.RequestMaxWait"),
	GThreadCallStackRequestMaxWait,
	TEXT("The number of seconds to spin before an individual back trace has timed out."));

float GThreadCallStackMaxWait = 5.0f;
static TAutoConsoleVariable<float> CVarAndroidPlatformThreadCallStackMaxWait(
	TEXT("AndroidPlatformThreadStackWalk.MaxWait"),
	GThreadCallStackMaxWait,
	TEXT("The number of seconds allowed to spin before killing the process, with the assumption the back trace handler has hung."));

#if ANDROID_HAS_RTSIGNALS
static FAsyncThreadBackTrace SignalThreadStackUserData;
/*Async stack backtracing capturing works needs to work in two modes:
 * 1. Serial - when we fire a request and wait for it (or time out) before firing next request.
 *    In case of a time out, serial request uses static SignalThreadStackUserData, so it's possible we can send a second request
 *    after the one that timed out and first request will be processed at this time, providing a wrong data to the second request.
 *    To prevent this from happening, we check if there's no serial request in flight before firing one
 * 2. Broadcast - when we fire a bunch of requests one ofter the other and then wait on all of them.
 *    This mode is not immune to an issue described before, but this is called only in a crash handler, so we are not going to fire a new request afterwards.
 */ 
static std::atomic<FAsyncThreadBackTrace*> InFlightSerialRequest(nullptr);

// the callback when THREAD_CALLSTACK_GENERATOR is being processed.
void FAndroidPlatformStackWalk::HandleBackTraceSignal(siginfo* Info, void* Context)
{
	FAsyncThreadBackTrace* BackTrace = (FAsyncThreadBackTrace*)Info->si_value.sival_ptr;
	BackTrace->Depth = FPlatformStackWalk::CaptureStackBackTrace(BackTrace->BackTrace, BackTrace->StackTraceMaxDepth, Context);
	BackTrace->Flag.store(1, std::memory_order_release);
	if (InFlightSerialRequest.load(std::memory_order_relaxed) == BackTrace)
	{
		InFlightSerialRequest.store(nullptr, std::memory_order_release);
	}
}

// Sends a signal to ThreadId, wait AndroidPlatformThreadStackWalk.RequestMaxWait seconds for result or time out and return 0.
// if callstack capture begins, but takes > AndroidPlatformThreadStackWalk.MaxWait the process will be killed.
// Is not thread safe, returns 0 if a CaptureThreadStackBackTrace is occurring on another thread.
uint32 FAndroidPlatformStackWalk::CaptureThreadStackBackTrace(uint64 ThreadId, uint64* BackTrace, uint32 MaxDepth, void* Context)
{
	static TAtomic<bool> bHasReentered(false);
	bool bExpected = false;
	if (!bHasReentered.CompareExchange(bExpected, true))
	{
		return 0;
	}
	ON_SCOPE_EXIT
	{
		bHasReentered = false;
	};

	if (InFlightSerialRequest.load(std::memory_order_acquire) != nullptr)
	{
		return 0;
	}

	SignalThreadStackUserData.Depth = 0;
	SignalThreadStackUserData.ThreadID = ThreadId;
	SignalThreadStackUserData.Flag.store(0, std::memory_order_release);

	auto WaitForSignalHandlerToFinishOrCrash = [BackTrace]()
	{
		const float PollTime = 0.001f;

		for (float CurrentTime = 0; CurrentTime <= GThreadCallStackRequestMaxWait; CurrentTime += PollTime)
		{
			if (SignalThreadStackUserData.Flag.load(std::memory_order_acquire))
			{
				FMemory::Memcpy(BackTrace, SignalThreadStackUserData.BackTrace, SignalThreadStackUserData.Depth * sizeof(*BackTrace));
				return SignalThreadStackUserData.Depth;
			}

			FPlatformProcess::SleepNoStats(PollTime);
		}

		// Time out
		return 0;
	};

	InFlightSerialRequest.store(&SignalThreadStackUserData, std::memory_order_relaxed);
	if (CaptureThreadStackBackTraceAsync(&SignalThreadStackUserData))
	{
		return WaitForSignalHandlerToFinishOrCrash();
	}
	
	InFlightSerialRequest.store(nullptr, std::memory_order_relaxed);
	return 0;
}

int FAndroidPlatformStackWalk::CaptureThreadStackBackTraceAsync(FAsyncThreadBackTrace* BackTrace)
{
	sigval UserData;
	UserData.sival_ptr = BackTrace;

	siginfo_t info;
	memset(&info, 0, sizeof(siginfo_t));
	info.si_signo = THREAD_CALLSTACK_GENERATOR;
	info.si_code = SI_QUEUE;
	info.si_pid = syscall(SYS_getpid);
	info.si_uid = syscall(SYS_getuid);
	info.si_value = UserData;

	// Avoid using sigqueue here as if the ThreadId is already blocked and in a signal handler
	// sigqueue will try a different thread signal handler and report the wrong callstack
	if (syscall(SYS_rt_tgsigqueueinfo, info.si_pid, BackTrace->ThreadID, THREAD_CALLSTACK_GENERATOR, &info) == 0)
	{
		return 1;
	}

	return 0;
}
#else
uint32 FAndroidPlatformStackWalk::CaptureThreadStackBackTrace(uint64 ThreadId, uint64* BackTrace, uint32 MaxDepth, void* Context)
{
	return 0;
}
#endif //ANDROID_HAS_RTSIGNALS