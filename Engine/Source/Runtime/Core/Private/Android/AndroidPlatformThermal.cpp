// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AndroidPlatformThermal.cpp: Android thermal API queries.
=============================================================================*/
#include "Android/AndroidPlatformThermal.h"
#include <sys/types.h>
#include <unistd.h>
#include <stdlib.h>
#include <limits.h>
#include <dlfcn.h>
#include "Containers/StringConv.h"
#include "CoreGlobals.h"
#include "Misc/App.h"
#include "Misc/Guid.h"
#include "HAL/PlatformStackWalk.h"
#include "Misc/Paths.h"
#include "GenericPlatform/GenericPlatformOutputDevices.h"
#include "HAL/FileManager.h"
#include "HAL/IConsoleManager.h"
#include "HAL/ThreadManager.h"
#include "HAL/RunnableThread.h"
#include "GenericPlatform/GenericPlatformCrashContext.h"
#include "Async/TaskGraphInterfaces.h"
#include "Android/AndroidJavaEnv.h"
#include <android/thermal.h>
#include "Android/AndroidJava.h"

#if USE_ANDROID_JNI && !USE_ANDROID_STANDALONE
namespace UE
{
	namespace Android
	{
		class FAndroidThermalManagerImpl
		{
			// *** From Google: 
			// * We recommend calling this every 10 seconds because Android devices do not much frequently update thermal data.
			// * In the case of the Pixel, it updates every 7 seconds, and the Galaxy is 10 secs.
			// * It's ok if you call it with multiple forecasts within 10 secs, but please never call it no more than twice per second.
			// ***
			// We set the period to 1 second to track the thermal status as tightly as we can.
			static constexpr float CallRateLimitSeconds = 1.0f;
			static constexpr int NumPeriods = FAndroidPlatformThermal::EForecastPeriod::NUM_FORECASTPERIODS;
			static constexpr int PeriodEnumToS[NumPeriods] = { 0, 1, 5, 10 };
			static_assert(NumPeriods == 4); // update PeriodEnumToS if this changes.
			double LastCallTime = 0;
			FCriticalSection ThermalCS;

			FAndroidPlatformThermal::EForecastPeriod PeriodToRead = FAndroidPlatformThermal::EForecastPeriod::INSTANT;

			FJavaClassObject ActivityClass;			// note: ActivityClass must always be accessed from the same thread that created it.
			FJavaClassMethod ThermalHeadroomMethod;

			FAndroidThermalManagerImpl() : ActivityClass(FJavaClassObject::GetGameActivity())
			{
				ThermalHeadroomMethod = ActivityClass.GetClassMethod("AndroidThunkJava_getThermalHeadroom", "(I)F");
			}

		public:
			static FAndroidThermalManagerImpl& Get()
			{
				static FAndroidThermalManagerImpl AndroidThermalManager;
				return AndroidThermalManager;				
			}

			struct FThermalInfo
			{
				float HeadroomForecasts[NumPeriods];
			}ThermalInfo;

			const FThermalInfo& UpdateAndGetThermalInfo()
			{
				float CurrentTime = (float)FPlatformTime::Seconds();
				if (CurrentTime > LastCallTime + CallRateLimitSeconds)
				{
					if (ThermalCS.TryLock())
					{
						float RecentThermalReading = ActivityClass.CallMethod<float>(ThermalHeadroomMethod, PeriodEnumToS[PeriodToRead]);
						// Bypass NaN values returned from getThermalHeadroom() because the device won't update thermal data frequently.
						if (!isnan(RecentThermalReading))
						{
							ThermalInfo.HeadroomForecasts[PeriodToRead] = RecentThermalReading;
						}

						LastCallTime = CurrentTime;
						PeriodToRead = (FAndroidPlatformThermal::EForecastPeriod)((int)(PeriodToRead+1) % NumPeriods);
						ThermalCS.Unlock();
					}
				}
				return ThermalInfo;
			}

		};
	}
}

float FAndroidPlatformThermal::GetThermalStress(FAndroidPlatformThermal::EForecastPeriod ForecastPeriod)
{
	int Idx = (int)ForecastPeriod;
	if (ensure(Idx >= 0 && Idx < (int)FAndroidPlatformThermal::EForecastPeriod::NUM_FORECASTPERIODS))
	{
		const UE::Android::FAndroidThermalManagerImpl::FThermalInfo& ThermalInfo = UE::Android::FAndroidThermalManagerImpl::Get().UpdateAndGetThermalInfo();
		return ThermalInfo.HeadroomForecasts[ForecastPeriod];
	}
	return -1;
}
#else
float FAndroidPlatformThermal::GetThermalStress(FAndroidPlatformThermal::EForecastPeriod ForecastPeriod) { return -1.0; }
#endif
