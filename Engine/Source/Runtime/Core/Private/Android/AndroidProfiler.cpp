// Copyright Epic Games, Inc. All Rights Reserved.

#include "Android/AndroidProfiler.h"

#if USE_ANDROID_JNI
#include "CoreGlobals.h"
#include "CoreMinimal.h"
#include "Android/AndroidJava.h"
#include "Android/AndroidJniAndroidProfilingProfilerAccessor.h"
#include "Android/AndroidProfiler.h"
#include "HAL/IConsoleManager.h"
#include "Misc/Paths.h"
#include "HAL/PlatformFileManager.h"
#include "GenericPlatform/GenericPlatformFile.h"

DEFINE_LOG_CATEGORY_STATIC(LogAndroidProfiling, Log, Log);

namespace AndroidProfiler
{
	static FAutoConsoleCommand AndroidProfileConsoleCommand(
		TEXT("android.profile"),
		TEXT("Invoke android's profiling api.\n")
		TEXT("Call with no arguments to echo usage to the log.\n")
		TEXT("Note: requires Android 15 or above (API 35).\n")
		TEXT("(this API is rate limited, to remove the rate limit use: 'adb shell device_config put profiling_testing rate_limiter.disabled true')"),
		FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
			{
				FString CombinedArgs;
				for (const FString& Argument : Args)
				{
					CombinedArgs += Argument + TEXT(" ");
				}
				FString ProfileName = FAndroidProfiler::StartCapture(CombinedArgs, [](const FAndroidProfiler::FProfileResults& Results)
				{
					if (!Results.FilePath.IsEmpty())
					{
						const FString DestPath = FPaths::ProfilingDir() / FPaths::GetCleanFilename(Results.FilePath);
						UE_CLOGF(!Results.Error.IsEmpty(), LogAndroidProfiling, Warning, "Profile %ls completed with log: %ls", *Results.ProfileName, *Results.Error);
						IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
						// move the file to public path so adb can get to it.
						bool bFileMoved = PlatformFile.MoveFile(*DestPath, *Results.FilePath);
						if (!bFileMoved)
						{
							// copy the file if the move fails.
							bFileMoved = PlatformFile.CopyFile(*DestPath, *Results.FilePath);
							PlatformFile.DeleteFile(*Results.FilePath);
						}

						UE_LOGF(LogAndroidProfiling, Log, "Profile %ls complete, retrieve via 'adb pull %ls'", *Results.ProfileName, *DestPath);
					}
					else
					{
						UE_LOGF(LogAndroidProfiling, Error, "Profile %ls failed: %ls", *Results.ProfileName, *Results.Error);
					}
				});
			}));

	static jclass ProfilerAccessorClassId = 0;
	static jmethodID IssueProfilerCommandMethod = 0;
	static jmethodID StopProfilerCommandMethod = 0;
	static void InitJNI();

	static bool IsJNIAvailable()
	{
		InitJNI();
		return ProfilerAccessorClassId && IssueProfilerCommandMethod && StopProfilerCommandMethod;
	}

	static void InitJNI()
	{
		static bool bInitJNI = false;
		if (bInitJNI)
		{
			return;
		}
		bInitJNI = true;

		if (JNIEnv* Env = AndroidJavaEnv::GetJavaEnv())
		{
			ProfilerAccessorClassId = UE::Jni::Class<UE::Jni::AndroidProfiling::FProfilerAccessor>;
			if(ProfilerAccessorClassId)
			{
				IssueProfilerCommandMethod = Env->GetStaticMethodID(ProfilerAccessorClassId, "AndroidThunkJava_IssueProfilerCommand", "(Ljava/lang/String;)[Ljava/lang/String;");
				StopProfilerCommandMethod = Env->GetStaticMethodID(ProfilerAccessorClassId, "AndroidThunkJava_StopProfilerCommand", "(Ljava/lang/String;)Z");
			}
		}
		UE_CLOGF(!IsJNIAvailable(), LogAndroidProfiling, Error, "JNI Could not find ProfilerAccessor class or methods. ProfilerAccessor is not supported when Compile SDK < 35.");
	}

	static FString CallProfiler(FString& ProfileNameOUT, const FString& CombinedArgs)
	{
		if (IsJNIAvailable())
		{
			JNIEnv* Env = AndroidJavaEnv::GetJavaEnv();

			auto CombinedArgsJava = FJavaHelper::ToJavaString(Env, CombinedArgs);
			TArray<FString> ProfilerResults = FJavaHelper::ObjectArrayToFStringTArray(Env,
				(jobjectArray)Env->CallStaticObjectMethod(ProfilerAccessorClassId, IssueProfilerCommandMethod, *CombinedArgsJava)
			);

			const bool bException = AndroidJavaEnv::CheckJavaException();
			UE_CLOGF(bException, LogAndroidProfiling, Error, "Profiler failed due to java exception.");

			if(ProfilerResults.Num() == 2)
			{
				ProfileNameOUT = ProfilerResults[0];
				const FString& ProfilerMessage = ProfilerResults[1];
				UE_LOGF(LogAndroidProfiling, Log, "Profiler : %ls ", ProfilerMessage.IsEmpty() ? TEXT("profile issued.") : *ProfilerMessage);
				return ProfilerMessage;
			}
			else
			{
				UE_LOGF(LogAndroidProfiling, Error, "Profiler failed.");
			}
		}
		else
		{
			UE_LOGF(LogAndroidProfiling, Error, "Profile failed, java profiler not found.");
		}
		return FString();
	}

	static bool CancelProfile(const FString& ProfileName)
	{
		if( IsJNIAvailable() )
		{
			JNIEnv* Env = AndroidJavaEnv::GetJavaEnv();
			auto ProfileNameJava = FJavaHelper::ToJavaString(Env, ProfileName);
			return (bool)Env->CallStaticBooleanMethod(ProfilerAccessorClassId, StopProfilerCommandMethod, *ProfileNameJava);
		}
		UE_LOGF(LogAndroidProfiling, Error, "Profile cancel failed, java profiler not found.");
		return false;
	}
}

// this provides a static function for accessing FAndroidProfiler just to avoid making OnProfileFinish part of the public api.
struct FAndroidProfilerInternal
{
	static void OnJavaProfileFinish(const FAndroidProfiler::FProfileResults& Results)
	{
		FAndroidProfiler::OnProfileFinish(Results);
	}
};

// called from jvm after profile completes.
void JNICALL UE::Jni::AndroidProfiling::FProfilerAccessor::nativeOnProfileFinish(JNIEnv* jenv, jclass clazz, jstring ProfileNameJava, jstring ProfileErrorJava, jstring ProfileFilepathJava)
{
	FAndroidProfiler::FProfileResults Results;
	Results.ProfileName = FJavaHelper::FStringFromParam(jenv, ProfileNameJava);
	Results.Error = FJavaHelper::FStringFromParam(jenv, ProfileErrorJava);
	Results.FilePath = FJavaHelper::FStringFromParam(jenv, ProfileFilepathJava);
	UE_LOGF(LogAndroidProfiling, Log, "nativeOnProfileFinish (%ls, %ls, %ls)",*Results.ProfileName, *Results.Error, *Results.FilePath);
	FAndroidProfilerInternal::OnJavaProfileFinish(Results);
}

// ProfilerCS is held during StartCapture, that completes before we lock and call OnFinish here.
void FAndroidProfiler::OnProfileFinish(const FAndroidProfiler::FProfileResults& Results)
{
	FScopeLock Lock(&ProfilerCS);
	FAndroidProfiler::FActiveSessions FoundSession = ActiveSessions.FindAndRemoveChecked(Results.ProfileName);
	FoundSession.OnFinish(Results);
}

FString FAndroidProfiler::StartCapture(const FString& Args, TUniqueFunction<void(const FAndroidProfiler::FProfileResults& Results)> OnFinish)
{
	FScopeLock Lock(&ProfilerCS);

	FString ProfileName;
	FString Message = AndroidProfiler::CallProfiler(ProfileName, Args);

	if (!ProfileName.IsEmpty())
	{
		check(!ActiveSessions.Contains(ProfileName));
		ActiveSessions.Add(ProfileName, FAndroidProfiler::FActiveSessions({ MoveTemp(OnFinish) }));
	}
	else
	{
		FAndroidProfiler::FProfileResults ProfileNameInProgressError;
		ProfileNameInProgressError.Error = FString::Printf(TEXT("Unable to launch profile. %s"), *Message);
		OnFinish(ProfileNameInProgressError);
	}
	return ProfileName;
}

void FAndroidProfiler::StopCapture(const FString& ProfileName)
{
	FScopeLock Lock(&ProfilerCS);
	if (ActiveSessions.Contains(ProfileName))
	{
		AndroidProfiler::CancelProfile(ProfileName);
	}
	else
	{
		UE_LOGF(LogAndroidProfiling, Error, "StopCapture ignored, profile %ls is not active.", *ProfileName);
	}
}
#else

FString FAndroidProfiler::StartCapture(const FString& Args, TUniqueFunction<void(const FAndroidProfiler::FProfileResults& Results)> OnFinish) { return FString(); }
void FAndroidProfiler::StopCapture(const FString& ProfileName) {}
void FAndroidProfiler::OnProfileFinish(const FAndroidProfiler::FProfileResults& Results) {}

#endif
