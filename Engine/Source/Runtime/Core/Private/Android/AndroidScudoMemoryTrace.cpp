// Copyright Epic Games, Inc. All Rights Reserved.

#include "AndroidScudoMemoryTrace.h"

#ifndef UE_MEMORY_TRACE_ANDROID_ENABLE_SCUDO_TRACING_SUPPORT
#	define UE_MEMORY_TRACE_ANDROID_ENABLE_SCUDO_TRACING_SUPPORT 0
#endif

#if UE_MEMORY_TRACE_ANDROID_ENABLE_SCUDO_TRACING_SUPPORT

#include "AndroidCallstackTrace.h"
#include "ProfilingDebugging/MemoryTrace.h"
#include "ScudoMemoryTrace.h"

#include <dlfcn.h>

extern bool MemoryTrace_IsActive();

static void ScudoMemoryTraceHookFn(EScudoEventType EventType, uint64_t Ptr, uint64_t Size, uint32_t Alignment, const void* Frames, uint32_t FrameCount)
{
	// LLM(UE_TRACE_METADATA_CLEAR_SCOPE());
	// LLM(UE_MEMSCOPE(ELLMTag::)); // TODO create new LLM tag

	const uint32 ExternalCallstackId = Frames ? CallstackTrace_GetExternalCallstackId((uint64*)Frames, FrameCount) : 0; 

	switch (EventType)
	{
	case EScudoEventType::Alloc:
	{
		MemoryTrace_Alloc(Ptr, Size, Alignment, EMemoryTraceRootHeap::SystemMemory, ExternalCallstackId);
		break;
	}
	case EScudoEventType::Free:
	{
		MemoryTrace_Free(Ptr, EMemoryTraceRootHeap::SystemMemory, ExternalCallstackId);
		break;
	}
	case EScudoEventType::Time:
	{
		// Time event before libUnreal.so, ignore
		break;
	}
	default:
	{
		break;
	}
	}
}

void AndroidScudoMemoryTrace::Init()
{
	if (!MemoryTrace_IsActive())
	{
		return;
	}

	// Find the set hook function dynamically, because libScudoMemoryTrace.so is LD_PRELOAD'ed
	decltype(&ScudoMemoryTrace_SetHook) SetHookFn = (decltype(&ScudoMemoryTrace_SetHook))dlsym(RTLD_DEFAULT, "ScudoMemoryTrace_SetHook");

	if (SetHookFn != nullptr)
	{
		SetHookFn(ScudoMemoryTraceHookFn);
	}
};

#else

void AndroidScudoMemoryTrace::Init() {}

#endif