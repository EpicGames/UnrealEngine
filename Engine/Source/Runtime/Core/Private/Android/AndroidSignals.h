// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AndroidSignals.h
=============================================================================*/
#include "Android/AndroidPlatformMisc.h"
#include "CoreGlobals.h"
#include "HAL/PlatformProcess.h"
#include <sys/syscall.h>

#pragma once

#if ANDROID_HAS_RTSIGNALS

#define THREAD_CALLSTACK_GENERATOR SIGRTMIN + 5
#define FATAL_SIGNAL_FWD SIGRTMIN + 6
#define THREADBACKTRACE_SIGNAL_FWD SIGRTMIN + 7

/*
	This is a helper class that attempts to avoid android's limited signal stack size by
	using a thread that infinite sleeps and is woken to only process signals.
	'Target' signals are caught and forwarded to the sleeping thread.
	The forwarded signal runs on the sleeping thread calling Derived::HandleTargetSignal() with the originating signal's parameters.

	The crash handler therefore, does not run on the same thread that originally received the target signal.

	Note, to avoid the requirement of suspending the thread while backgrounding (and therefore missing signals) the thread sleeps infinitely.
*/

extern float GAndroidSignalTimeOut;

template<typename Derived>
class FSignalHandler
{
protected:
	enum class ESignalType : int32 { Signal, Exit };
	enum class ESignalThreadStatus : int32 { NotInitialized, Ready, Starting, Busy, Complete, Exited };

	static void Init(int ForwardingSignal)
	{
		check(ForwardingSignalType == -1);
		check(SignalThreadStatus == (int32)ESignalThreadStatus::NotInitialized);

		ForwardingSignalType = ForwardingSignal;
		pthread_t SignalHandlerThread;

		pthread_attr_t otherAttr;
		pthread_attr_init(&otherAttr);
		pthread_attr_setdetachstate(&otherAttr, PTHREAD_CREATE_DETACHED);
		const size_t StackSize = 256 * 1024;
		bool bStackSizeSuccess = (pthread_attr_setstacksize(&otherAttr, StackSize) == 0);
		UE_CLOGF(!bStackSizeSuccess, LogAndroid, Error, "Failed to set signal thread stack size.");

		bool bThreadCreated = pthread_create(&SignalHandlerThread, &otherAttr, ThreadFunc, nullptr) == 0;
		UE_CLOGF(!bThreadCreated, LogAndroid, Fatal, "Failed to create signal handler");

		while (FPlatformAtomics::AtomicRead(&SignalThreadStatus) != (int32)ESignalThreadStatus::Ready)
		{
			FPlatformProcess::SleepNoStats(0);
		}

		struct sigaction ActionForForwardSignal;

		FMemory::Memzero(ActionForForwardSignal);
		sigfillset(&ActionForForwardSignal.sa_mask);
		ActionForForwardSignal.sa_flags = SA_SIGINFO | SA_RESTART;
		ActionForForwardSignal.sa_sigaction = &OnForwardedTargetSignal;
		sigaction(ForwardingSignalType, &ActionForForwardSignal, &PreviousActionForForwardSignal);
	}

	static void Release()
	{
		if (ForwardingSignalType != -1)
		{
			SendExitSignal();
			while (FPlatformAtomics::AtomicRead(&SignalThreadStatus) != (int32)ESignalThreadStatus::Exited)
			{
				FPlatformProcess::SleepNoStats(0);
			}

			sigaction(ForwardingSignalType, &PreviousActionForForwardSignal, nullptr);

			SignalThreadStatus = (int32)ESignalThreadStatus::NotInitialized;
			ForwardingThreadID = 0xffffffff;
			ForwardingSignalType = -1;
		}
	}

	static void ForwardSignal(int InSignal, siginfo* InInfo, void* InContext)
	{
		while (FPlatformAtomics::InterlockedCompareExchange(&SignalThreadStatus, (int32)ESignalThreadStatus::Starting, (int32)ESignalThreadStatus::Ready) != (int32)ESignalThreadStatus::Ready)
		{
			FPlatformProcess::SleepNoStats(0.0f);
		}
		SignalParams.Signal = InSignal;
		SignalParams.ThreadID = FPlatformTLS::GetCurrentThreadId();
		SignalParams.Info = InInfo;
		SignalParams.Context = InContext;

		FPlatformMisc::MemoryBarrier();

		if (SendSignal((int)ESignalType::Signal))
		{
			WaitForSignalHandlerToFinishOrExit();
		}
		else
		{
			// did not send, thread not available... set status to ready.
			FPlatformAtomics::AtomicStore(&SignalThreadStatus, (int32)ESignalThreadStatus::Ready);
		}
	}

private:
	static void* ThreadFunc(void* param)
	{
		ForwardingThreadID = FPlatformTLS::GetCurrentThreadId();
		FPlatformAtomics::AtomicStore(&SignalThreadStatus, (int32)ESignalThreadStatus::Ready);
		FPlatformProcess::SleepInfinite();
		return nullptr;
	}

	static bool SendSignal(int SignalType)
	{
		siginfo_t info;
		memset(&info, 0, sizeof(siginfo_t));
		info.si_signo = ForwardingSignalType;
		info.si_code = SI_QUEUE;
		info.si_pid = syscall(SYS_getpid);
		info.si_uid = syscall(SYS_getuid);
		info.si_value.sival_int = SignalType;

		// Avoid using sigqueue here as if the ThreadId is already blocked and in a signal handler
		// sigqueue will try a different thread signal handler and report the wrong callstack
		return syscall(SYS_rt_tgsigqueueinfo, info.si_pid, ForwardingThreadID, ForwardingSignalType, &info) == 0;
	}

	static bool SendExitSignal()
	{
		// enqueue a signal to exit the signal thread.
		return SendSignal((int)ESignalType::Exit);
	}

	static void WaitForSignalHandlerToFinishOrExit()
	{
		const float PollInterval = 0.01f;
		float CurrentWaitTime = 0.0f;

		while (FPlatformAtomics::InterlockedCompareExchange(&SignalThreadStatus, (int32)ESignalThreadStatus::Ready, (int32)ESignalThreadStatus::Complete) != (int32)ESignalThreadStatus::Complete)
		{
			// have we timed out?
			if (CurrentWaitTime > GAndroidSignalTimeOut)
			{
				exit(0);
			}
			FPlatformProcess::SleepNoStats(PollInterval);
			CurrentWaitTime += PollInterval;
		}
	}

	static void OnForwardedTargetSignal(int Signal, siginfo* Info, void* Context)
	{
		FPlatformAtomics::AtomicStore(&SignalThreadStatus, (int32)ESignalThreadStatus::Busy);

		if (Info->si_value.sival_int == (int)ESignalType::Exit)
		{
			FPlatformAtomics::AtomicStore(&SignalThreadStatus, (int32)ESignalThreadStatus::Exited);
			pthread_exit(0);
			return;
		}
		Derived::HandleTargetSignal(SignalParams.Signal, SignalParams.Info, SignalParams.Context, SignalParams.ThreadID);
		FPlatformAtomics::AtomicStore(&SignalThreadStatus, (int32)ESignalThreadStatus::Complete);
	}

	struct FSignalParams
	{
		int Signal;
		uint32 ThreadID;
		siginfo* Info;
		void* Context;
	};

	static FSignalParams SignalParams;
	static int32 SignalThreadStatus;
	static uint32 ForwardingThreadID;
	static int32 ForwardingSignalType;
	static struct sigaction PreviousActionForForwardSignal;
};
#endif