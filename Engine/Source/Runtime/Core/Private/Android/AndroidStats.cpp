// Copyright Epic Games, Inc. All Rights Reserved.

#include "Android/AndroidStats.h"
#include "Async/TaskGraphInterfaces.h"
#include "CoreMinimal.h"
#include "Android/AndroidPlatformMisc.h"
#include "Stats/Stats.h"
#include "Stats/ThermalStats.h"
#include "HAL/IConsoleManager.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include <android/thermal.h>

DEFINE_LOG_CATEGORY_STATIC(LogAndroidStats, Log, Log);

#if !UE_BUILD_SHIPPING && PLATFORM_ANDROID_ARM64
#	define HWCPIPE_SUPPORTED 1 
//#	define HWCPIPE_SUPPORTED 0 No longer disabling for now; we were seeing multiple missing frames since 36.00, starting since CL 42524925
//                             Keep this in place in-case we need to fix this again at a later time.
#else
#	define HWCPIPE_SUPPORTED 0
#endif

#if HWCPIPE_SUPPORTED
#include "libGPUCounters.h"
#endif
#include "Android/AndroidPlatformThermal.h"
#include "Tasks/Task.h"

DECLARE_STATS_GROUP(TEXT("Android CPU stats"), STATGROUP_AndroidCPU, STATCAT_Advanced);
CSV_DEFINE_CATEGORY(AndroidCPU, true);
CSV_DEFINE_CATEGORY(AndroidMemory, true);

DECLARE_DWORD_COUNTER_STAT(TEXT("Num Frequency Groups"), STAT_NumFreqGroups, STATGROUP_AndroidCPU);
DECLARE_DWORD_COUNTER_STAT(TEXT("Freq Group 0 : Max frequency (MHz)"), STAT_FreqGroup0MaxFrequency, STATGROUP_AndroidCPU);
DECLARE_DWORD_COUNTER_STAT(TEXT("Freq Group 0 : Min frequency (MHz)"), STAT_FreqGroup0MinFrequency, STATGROUP_AndroidCPU);
DECLARE_DWORD_COUNTER_STAT(TEXT("Freq Group 0 : Current frequency (MHz)"), STAT_FreqGroup0CurrentFrequency, STATGROUP_AndroidCPU);
DECLARE_FLOAT_COUNTER_STAT(TEXT("Freq Group 0 : Current frequency (% from Max)"), STAT_FreqGroup0CurrentFrequencyPercentage, STATGROUP_AndroidCPU);
DECLARE_DWORD_COUNTER_STAT(TEXT("Freq Group 0 : Num Cores"), STAT_FreqGroup0NumCores, STATGROUP_AndroidCPU);
CSV_DEFINE_STAT(AndroidCPU, CPUFreqMHzGroup0);
CSV_DEFINE_STAT(AndroidCPU, CPUFreqPercentageGroup0);

DECLARE_DWORD_COUNTER_STAT(TEXT("Freq Group 1 : Max frequency (MHz)"), STAT_FreqGroup1MaxFrequency, STATGROUP_AndroidCPU);
DECLARE_DWORD_COUNTER_STAT(TEXT("Freq Group 1 : Min frequency (MHz)"), STAT_FreqGroup1MinFrequency, STATGROUP_AndroidCPU);
DECLARE_DWORD_COUNTER_STAT(TEXT("Freq Group 1 : Current frequency (MHz)"), STAT_FreqGroup1CurrentFrequency, STATGROUP_AndroidCPU);
DECLARE_FLOAT_COUNTER_STAT(TEXT("Freq Group 1 : Current frequency (% from Max)"), STAT_FreqGroup1CurrentFrequencyPercentage, STATGROUP_AndroidCPU);
DECLARE_DWORD_COUNTER_STAT(TEXT("Freq Group 1 : Num Cores"), STAT_FreqGroup1NumCores, STATGROUP_AndroidCPU);
CSV_DEFINE_STAT(AndroidCPU, CPUFreqMHzGroup1);
CSV_DEFINE_STAT(AndroidCPU, CPUFreqPercentageGroup1);

DECLARE_DWORD_COUNTER_STAT(TEXT("Freq Group 2 : Max frequency (MHz)"), STAT_FreqGroup2MaxFrequency, STATGROUP_AndroidCPU);
DECLARE_DWORD_COUNTER_STAT(TEXT("Freq Group 2 : Min frequency (MHz)"), STAT_FreqGroup2MinFrequency, STATGROUP_AndroidCPU);
DECLARE_DWORD_COUNTER_STAT(TEXT("Freq Group 2 : Current frequency (MHz)"), STAT_FreqGroup2CurrentFrequency, STATGROUP_AndroidCPU);
DECLARE_FLOAT_COUNTER_STAT(TEXT("Freq Group 2 : Current frequency (% from Max)"), STAT_FreqGroup2CurrentFrequencyPercentage, STATGROUP_AndroidCPU);
DECLARE_DWORD_COUNTER_STAT(TEXT("Freq Group 2 : Num Cores"), STAT_FreqGroup2NumCores, STATGROUP_AndroidCPU);
CSV_DEFINE_STAT(AndroidCPU, CPUFreqMHzGroup2);
CSV_DEFINE_STAT(AndroidCPU, CPUFreqPercentageGroup2);

DECLARE_DWORD_COUNTER_STAT(TEXT("Freq Group 3 : Max frequency (MHz)"), STAT_FreqGroup3MaxFrequency, STATGROUP_AndroidCPU);
DECLARE_DWORD_COUNTER_STAT(TEXT("Freq Group 3 : Min frequency (MHz)"), STAT_FreqGroup3MinFrequency, STATGROUP_AndroidCPU);
DECLARE_DWORD_COUNTER_STAT(TEXT("Freq Group 3 : Current frequency (MHz)"), STAT_FreqGroup3CurrentFrequency, STATGROUP_AndroidCPU);
DECLARE_FLOAT_COUNTER_STAT(TEXT("Freq Group 3 : Current frequency (% from Max)"), STAT_FreqGroup3CurrentFrequencyPercentage, STATGROUP_AndroidCPU);
DECLARE_DWORD_COUNTER_STAT(TEXT("Freq Group 3 : Num Cores"), STAT_FreqGroup3NumCores, STATGROUP_AndroidCPU);
CSV_DEFINE_STAT(AndroidCPU, CPUFreqMHzGroup3);
CSV_DEFINE_STAT(AndroidCPU, CPUFreqPercentageGroup3);

DECLARE_DWORD_COUNTER_STAT(TEXT("Num CPU Cores"), STAT_NumCPUCores, STATGROUP_AndroidCPU);

DECLARE_FLOAT_COUNTER_STAT(TEXT("Freq Group 0 : highest core utilization %"), STAT_FreqGroup0MaxUtilization, STATGROUP_AndroidCPU);
DECLARE_FLOAT_COUNTER_STAT(TEXT("Freq Group 1 : highest core utilization %"), STAT_FreqGroup1MaxUtilization, STATGROUP_AndroidCPU);
DECLARE_FLOAT_COUNTER_STAT(TEXT("Freq Group 2 : highest core utilization %"), STAT_FreqGroup2MaxUtilization, STATGROUP_AndroidCPU);
DECLARE_FLOAT_COUNTER_STAT(TEXT("Freq Group 3 : highest core utilization %"), STAT_FreqGroup3MaxUtilization, STATGROUP_AndroidCPU);

CSV_DEFINE_STAT(AndroidCPU, CPUTemp);
DECLARE_FLOAT_COUNTER_STAT(TEXT("CPU Temperature"), STAT_CPUTemp, STATGROUP_AndroidCPU);

CSV_DEFINE_STAT(AndroidCPU, ThermalStatus);
DECLARE_DWORD_COUNTER_STAT(TEXT("Thermal Status"), STAT_ThermalStatus, STATGROUP_AndroidCPU);

CSV_DEFINE_STAT(AndroidCPU, ThermalStress);
DECLARE_FLOAT_COUNTER_STAT(TEXT("Thermal Stress"), STAT_ThermalStress, STATGROUP_AndroidCPU);

#if CSV_PROFILER_STATS

#define CSV_STAT_PTR(StatName)									&_GCsvStat_##StatName
#define CSV_CUSTOM_STAT_DEFINED_BY_PTR(StatPtr,Value,Op)		FCsvProfiler::RecordCustomStat(StatPtr->Name, StatPtr->CategoryIndex, Value, Op);
FCsvDeclaredStat* GCPUFreqStats[] = {
	CSV_STAT_PTR(CPUFreqMHzGroup0),
	CSV_STAT_PTR(CPUFreqMHzGroup1),
	CSV_STAT_PTR(CPUFreqMHzGroup2),
	CSV_STAT_PTR(CPUFreqMHzGroup3)
};

FCsvDeclaredStat* GCPUFreqPercentageStats[] = {
	CSV_STAT_PTR(CPUFreqPercentageGroup0),
	CSV_STAT_PTR(CPUFreqPercentageGroup1),
	CSV_STAT_PTR(CPUFreqPercentageGroup2),
	CSV_STAT_PTR(CPUFreqPercentageGroup3)
};
#undef CSV_STAT_PTR

#else
#define CSV_CUSTOM_STAT_DEFINED_BY_PTR(StatPtr,Value,Op)
#endif

static void InitGPUStats();
static void UpdateGPUStats();
static void LogGPUStats();

static float GAndroidCPUStatsUpdateRate = 0.100f;
static FAutoConsoleVariableRef CVarAndroidCollectCPUStatsRate(
	TEXT("Android.CPUStatsUpdateRate"),
	GAndroidCPUStatsUpdateRate,
	TEXT("Update rate in seconds for collecting CPU Stats (Default: 0.1)\n")
	TEXT("0 to disable."),
	ECVF_Default);

static int GAndroidHWCPipeStatsEnabled = 1;
static FAutoConsoleVariableRef CVarAndroidHWCPipeStatsEnabled(
	TEXT("Android.HWCPipeStatsEnabled"),
	GAndroidHWCPipeStatsEnabled,
	TEXT("Log GPU statistics using HWCPipe/libGPUCounters (Default: 1)"),
	ECVF_Default);


static int GThermalStatus = 0;
static int GTrimMemoryBackgroundLevel = 0;
static int GTrimMemoryForegroundLevel = 0;

CSV_DEFINE_STAT(AndroidMemory, TrimMemoryBackgroundLevel);
CSV_DEFINE_STAT(AndroidMemory, TrimMemoryForegroundLevel);

CSV_DEFINE_STAT(AndroidMemory, Mem_RSS);
CSV_DEFINE_STAT(AndroidMemory, Mem_Swap);
CSV_DEFINE_STAT(AndroidMemory, Mem_TotalUsed);


#if CSV_PROFILER || STATS
namespace UE
{
	namespace Android
	{
		constexpr uint32 MaxFrequencyGroupStats = 4;

		struct FCPUStatValues
		{
			float CPUTemp = 0.0f;
#if CSV_PROFILER
			int ThermalStatus = 0;
			float Thermals5S = 0;
			int TrimMemoryBackgroundLevel = 0;
			int TrimMemoryForegroundLevel = 0;
			FPlatformMemoryStats MemStats;
#endif
			struct FFrequencyGroupStats
			{
				int CoreCount = 0;
				int MaxFrequency = 0;
				int CurrentFrequency = 0.0f;
				float MaxSingleCoreUtilization = 0.0f;
			};
			int NumFrequencyGroups = 0;
			FFrequencyGroupStats FrequencyGroupStats[UE::Android::MaxFrequencyGroupStats];

#if STATS
			static inline const FName AndroidFrequencyGroupMaxFreqStats[] = {
				GET_STATFNAME(STAT_FreqGroup0MaxFrequency),
				GET_STATFNAME(STAT_FreqGroup1MaxFrequency),
				GET_STATFNAME(STAT_FreqGroup2MaxFrequency),
				GET_STATFNAME(STAT_FreqGroup3MaxFrequency),
			};

			static inline const FName AndroidFrequencyGroupMinFreqStats[] = {
				GET_STATFNAME(STAT_FreqGroup0MinFrequency),
				GET_STATFNAME(STAT_FreqGroup1MinFrequency),
				GET_STATFNAME(STAT_FreqGroup2MinFrequency),
				GET_STATFNAME(STAT_FreqGroup3MinFrequency),
			};

			static inline const FName AndroidFrequencyGroupCurrentFreqStats[] = {
				GET_STATFNAME(STAT_FreqGroup0CurrentFrequency),
				GET_STATFNAME(STAT_FreqGroup1CurrentFrequency),
				GET_STATFNAME(STAT_FreqGroup2CurrentFrequency),
				GET_STATFNAME(STAT_FreqGroup3CurrentFrequency),
			};


			static inline const FName AndroidFrequencyGroupCurrentFreqPercentageStats[] = {
				GET_STATFNAME(STAT_FreqGroup0CurrentFrequencyPercentage),
				GET_STATFNAME(STAT_FreqGroup1CurrentFrequencyPercentage),
				GET_STATFNAME(STAT_FreqGroup2CurrentFrequencyPercentage),
				GET_STATFNAME(STAT_FreqGroup3CurrentFrequencyPercentage),
			};

			static inline const FName AndroidFrequencyGroupNumCoresStats[] = {
				GET_STATFNAME(STAT_FreqGroup0NumCores),
				GET_STATFNAME(STAT_FreqGroup1NumCores),
				GET_STATFNAME(STAT_FreqGroup2NumCores),
				GET_STATFNAME(STAT_FreqGroup3NumCores),
			};

			static inline const FName AndroidFrequencyGroupMaxCoresUtilizationStats[] = {
				GET_STATFNAME(STAT_FreqGroup0MaxUtilization),
				GET_STATFNAME(STAT_FreqGroup1MaxUtilization),
				GET_STATFNAME(STAT_FreqGroup2MaxUtilization),
				GET_STATFNAME(STAT_FreqGroup3MaxUtilization),
			};

#endif
			void PushStats() const
			{
#if CSV_PROFILER
				CSV_CUSTOM_STAT_DEFINED(CPUTemp, CPUTemp, ECsvCustomStatOp::Set);
				CSV_CUSTOM_STAT_DEFINED(ThermalStatus, GThermalStatus, ECsvCustomStatOp::Set);
				CSV_CUSTOM_STAT_DEFINED(ThermalStress, Thermals5S, ECsvCustomStatOp::Set);
				CSV_CUSTOM_STAT_DEFINED(TrimMemoryForegroundLevel, GTrimMemoryForegroundLevel, ECsvCustomStatOp::Set);
				CSV_CUSTOM_STAT_DEFINED(TrimMemoryBackgroundLevel, GTrimMemoryBackgroundLevel, ECsvCustomStatOp::Set);
				CSV_CUSTOM_STAT_DEFINED(Mem_Swap, (int)(MemStats.VMSwap / (1024 * 1024)), ECsvCustomStatOp::Set);
				CSV_CUSTOM_STAT_DEFINED(Mem_RSS, (int)(MemStats.VMRss / (1024 * 1024)), ECsvCustomStatOp::Set);
				CSV_CUSTOM_STAT_DEFINED(Mem_TotalUsed, (int)(MemStats.UsedPhysical / (1024 * 1024)), ECsvCustomStatOp::Set);
				for (int32 FrequencyGroupIndex = 0; FrequencyGroupIndex < NumFrequencyGroups; FrequencyGroupIndex++)
				{
					const FFrequencyGroupStats& Stats = FrequencyGroupStats[FrequencyGroupIndex];
					CSV_CUSTOM_STAT_DEFINED_BY_PTR(GCPUFreqStats[FrequencyGroupIndex], Stats.CurrentFrequency, ECsvCustomStatOp::Set);
					const float CurrentFrequencyPercentage = ((float)Stats.CurrentFrequency / (float)Stats.MaxFrequency) * 100.0f;
					CSV_CUSTOM_STAT_DEFINED_BY_PTR(GCPUFreqPercentageStats[FrequencyGroupIndex], CurrentFrequencyPercentage, ECsvCustomStatOp::Set);
				}
#endif
#if STATS
				static const FName CPUStatName = GET_STATFNAME(STAT_CPUTemp);
				static const FName ThermalStatusStatName = GET_STATFNAME(STAT_ThermalStatus);
				SET_FLOAT_STAT_FName(CPUStatName, CPUTemp);
				SET_DWORD_STAT_FName(ThermalStatusStatName, GThermalStatus);

				for (int32 FrequencyGroupIndex = 0; FrequencyGroupIndex < NumFrequencyGroups; FrequencyGroupIndex++)
				{
					const FFrequencyGroupStats& Stats = FrequencyGroupStats[FrequencyGroupIndex];
					const float CurrentFrequencyPercentage = ((float)Stats.CurrentFrequency / (float)Stats.MaxFrequency) * 100.0f;

					SET_DWORD_STAT_FName(AndroidFrequencyGroupMaxFreqStats[FrequencyGroupIndex], Stats.MaxFrequency);
					SET_DWORD_STAT_FName(AndroidFrequencyGroupNumCoresStats[FrequencyGroupIndex], Stats.CoreCount);
					//SET_DWORD_STAT_FName(AndroidFrequencyGroupMinFreqStats[FrequencyGroupIndex], FrequencyGroup.MinFrequency);
					SET_DWORD_STAT_FName(AndroidFrequencyGroupCurrentFreqStats[FrequencyGroupIndex], Stats.CurrentFrequency);
					SET_FLOAT_STAT_FName(AndroidFrequencyGroupCurrentFreqPercentageStats[FrequencyGroupIndex], CurrentFrequencyPercentage);
					SET_FLOAT_STAT_FName(AndroidFrequencyGroupMaxCoresUtilizationStats[FrequencyGroupIndex], Stats.MaxSingleCoreUtilization);
				}
#endif
			}

			static FCPUStatValues& Get()
			{
				static FCPUStatValues CSVStatValues;
				return CSVStatValues;
			}
		};
	}
}
#endif // CSV_PROFILER || STATS

void FAndroidStats::OnThermalStatusChanged(int Status)
{
	GThermalStatus = Status;
}

void FAndroidStats::OnTrimMemory(int TrimLevel)
{
	// https://developer.android.com/reference/android/content/ComponentCallbacks2#constants_1
	enum ETrimLevel
	{
		TRIM_MEMORY_BACKGROUND = 40,
		TRIM_MEMORY_COMPLETE = 80,
		TRIM_MEMORY_MODERATE = 60,
		TRIM_MEMORY_RUNNING_CRITICAL = 15,
		TRIM_MEMORY_RUNNING_LOW = 10,
		TRIM_MEMORY_RUNNING_MODERATE = 5,
		TRIM_MEMORY_UI_HIDDEN = 20,
	};

	GTrimMemoryBackgroundLevel = 0;
	GTrimMemoryForegroundLevel = 0;

	switch (TrimLevel)
	{
	case TRIM_MEMORY_UI_HIDDEN:
		GTrimMemoryBackgroundLevel = 1;
		break;
	case TRIM_MEMORY_BACKGROUND:
		GTrimMemoryBackgroundLevel = 2;
		break;
	case TRIM_MEMORY_MODERATE:
		GTrimMemoryBackgroundLevel = 3;
		break;
	case TRIM_MEMORY_COMPLETE:
		GTrimMemoryBackgroundLevel = 4;
		break;

	case TRIM_MEMORY_RUNNING_LOW:
		GTrimMemoryForegroundLevel = 1;
		break;
	case TRIM_MEMORY_RUNNING_MODERATE:
		GTrimMemoryForegroundLevel = 2;
		break;
	case TRIM_MEMORY_RUNNING_CRITICAL:
		GTrimMemoryForegroundLevel = 3;
		break;
	default:
		GTrimMemoryForegroundLevel = -1;
		GTrimMemoryBackgroundLevel = -1;
		break;
}
}

static std::atomic<bool> GIsStatTaskActive = false;

void FAndroidStats::Init()
{
	InitGPUStats();
}

void FAndroidStats::LogGPUStats()
{
	::LogGPUStats();
}

// We want to mirror the android enum values, so check some of them.
static_assert((int)EDeviceThermalState::None == (int)ATHERMAL_STATUS_NONE);
static_assert((int)EDeviceThermalState::Severe == (int)ATHERMAL_STATUS_SEVERE);
EDeviceThermalState FAndroidStats::GetThermalStatus()
{
	return static_cast<EDeviceThermalState>(GThermalStatus);
}


void FAndroidStats::UpdateAndroidStats()
{
#if CSV_PROFILER || STATS
	UpdateGPUStats();
	SCOPED_NAMED_EVENT(UpdateAndroidStats, FColor::Green);

	if (GAndroidCPUStatsUpdateRate <= 0.0f)
	{
		return;
	}

	static UE::FMutex CSVStatsLock;

	{
		/*
		*   FCPUStatValues contains the state that is pushed each frame to the CSV and stats systems.
		*	The state is periodically updated via a background thread to avoid potential stalls on the GT.
		*/
		UE::TUniqueLock Lock(CSVStatsLock);
		UE::Android::FCPUStatValues::Get().PushStats();
		UE::ThermalStats::GetAndPushStats();
	}

	static uint64 LastCollectionTime = FPlatformTime::Cycles64();
	const uint64 CurrentTime = FPlatformTime::Cycles64();
	const bool bUpdateStats = ((FPlatformTime::ToSeconds64(CurrentTime - LastCollectionTime) >= GAndroidCPUStatsUpdateRate));

	bool bExpected = false;
	if (!bUpdateStats || !GIsStatTaskActive.compare_exchange_weak(bExpected, true))
	{
		return;
	}
	LastCollectionTime = CurrentTime;

	// Run everything in a background task thread so that long system calls won't lock up game thread
	UE::Tasks::Launch(UE_SOURCE_LOCATION, []()
	{
		SCOPED_NAMED_EVENT(UpdateAndroidStats, FColor::Green);

		constexpr int32 MaxCoresStatsSupport = 16;
		int32 NumCores = FMath::Min(FAndroidMisc::NumberOfCores(), MaxCoresStatsSupport);
		ensure(FAndroidMisc::NumberOfCores() <= MaxCoresStatsSupport);
		struct FFrequencyGroup
		{
			uint32 MinFrequency;
			uint32 MaxFrequency;
			uint32 CoreCount;
		};

		static uint32 UnInitializedCores = NumCores;
		static TArray<FFrequencyGroup> FrequencyGroups;
		static uint32 CoreFrequencyGroupIndex[MaxCoresStatsSupport] = {
			0xFFFFFFFF, 0xFFFFFFFF,0xFFFFFFFF,0xFFFFFFFF,
			0xFFFFFFFF, 0xFFFFFFFF,0xFFFFFFFF,0xFFFFFFFF,
			0xFFFFFFFF, 0xFFFFFFFF,0xFFFFFFFF,0xFFFFFFFF,
			0xFFFFFFFF, 0xFFFFFFFF,0xFFFFFFFF,0xFFFFFFFF,
		};

		if (UnInitializedCores != 0)
		{
			for (int32 CoreIndex = 0; CoreIndex < NumCores; CoreIndex++)
			{
				if (CoreFrequencyGroupIndex[CoreIndex] == 0xFFFFFFFF)
				{
					uint32 MinFreq = FAndroidMisc::GetCoreFrequency(CoreIndex, FAndroidMisc::ECoreFrequencyProperty::MinFrequency) / 1000;
					uint32 MaxFreq = FAndroidMisc::GetCoreFrequency(CoreIndex, FAndroidMisc::ECoreFrequencyProperty::MaxFrequency) / 1000;
					if (MaxFreq > 0)
					{
						UnInitializedCores--;
						int32 FoundIndex = FrequencyGroups.IndexOfByPredicate([&MinFreq, &MaxFreq](const FFrequencyGroup& s) { return s.MinFrequency == MinFreq && s.MaxFrequency == MaxFreq; });
						if (FoundIndex == INDEX_NONE)
						{
							FFrequencyGroup NewGroup = { MinFreq,MaxFreq, 1 };
							CoreFrequencyGroupIndex[CoreIndex] = FrequencyGroups.Add(NewGroup);
						}
						else
						{
							CoreFrequencyGroupIndex[CoreIndex] = FoundIndex;
							FrequencyGroups[FoundIndex].CoreCount++;
						}
					}
				}
			}
		}

		auto GetFrequencyGroupCurrentFrequency = [&](int32 FrequencyGroupIdx)
		{
			for (int32 CoreIdx = 0; CoreIdx < NumCores; CoreIdx++)
			{
				if (CoreFrequencyGroupIndex[CoreIdx] == FrequencyGroupIdx)
				{
					return FAndroidMisc::GetCoreFrequency(CoreIdx, FAndroidMisc::ECoreFrequencyProperty::CurrentFrequency) / 1000;
				}
			}
			return 0u;
		};

		check(FrequencyGroups.Num() <= UE::Android::MaxFrequencyGroupStats);
#if CSV_PROFILER || STATS
		{
			UE::TUniqueLock Lock(CSVStatsLock);
			UE::Android::FCPUStatValues& CSVStatValues = UE::Android::FCPUStatValues::Get();

			CSVStatValues.CPUTemp = FAndroidMisc::GetCPUTemperature();
#if CSV_PROFILER 
			CSVStatValues.Thermals5S = FAndroidPlatformThermal::GetThermalStress(FAndroidPlatformThermal::EForecastPeriod::FIVE_SEC);
			CSVStatValues.ThermalStatus = GThermalStatus;
			CSVStatValues.TrimMemoryBackgroundLevel = GTrimMemoryBackgroundLevel;
			CSVStatValues.TrimMemoryForegroundLevel = GTrimMemoryForegroundLevel;
			CSVStatValues.MemStats = FAndroidPlatformMemory::GetStats();
#endif //#if CSV_PROFILER
		}
#endif //#if CSV_PROFILER || STATS

		float MaxSingleCoreUtilization[UE::Android::MaxFrequencyGroupStats] = { 0.0f };
		{
			FAndroidMisc::FCPUState& AndroidCPUState = FAndroidMisc::GetCPUState();
			for (int32 CoreIndex = 0; CoreIndex < NumCores; CoreIndex++)
			{
				uint32 FrequencyGroupIndex = CoreFrequencyGroupIndex[CoreIndex];
				if (FrequencyGroupIndex != 0xFFFFFFFF)
				{
					float& MaxCoreUtilization = MaxSingleCoreUtilization[FrequencyGroupIndex];
					MaxCoreUtilization = FMath::Max((float)AndroidCPUState.Utilization[CoreIndex], MaxCoreUtilization);
				}
			}
		}

		{
			UE::TUniqueLock Lock(CSVStatsLock);
			UE::Android::FCPUStatValues& CSVStatValues = UE::Android::FCPUStatValues::Get();

			CSVStatValues.NumFrequencyGroups = FrequencyGroups.Num();
			for (int32 FrequencyGroupIndex = 0; FrequencyGroupIndex < FrequencyGroups.Num(); FrequencyGroupIndex++)
			{
				UE::Android::FCPUStatValues::FFrequencyGroupStats& FrequencyGroupStats = CSVStatValues.FrequencyGroupStats[FrequencyGroupIndex];
				const FFrequencyGroup& FrequencyGroup = FrequencyGroups[FrequencyGroupIndex];
				FrequencyGroupStats.CoreCount = FrequencyGroup.CoreCount;
				FrequencyGroupStats.MaxFrequency = FrequencyGroup.MaxFrequency;
				FrequencyGroupStats.CurrentFrequency = GetFrequencyGroupCurrentFrequency(FrequencyGroupIndex);
				FrequencyGroupStats.MaxSingleCoreUtilization = MaxSingleCoreUtilization[FrequencyGroupIndex];
			}
		}

		UpdateGPUStats();
		GIsStatTaskActive = false;
	}, UE::Tasks::ETaskPriority::BackgroundLow);
#endif

}


#if HWCPIPE_SUPPORTED

static void GPUStatsLogCallback(uint8_t Level, const char* Message)
{
	switch (static_cast<libGPUCountersLogLevel>(Level))
	{
		case libGPUCountersLogLevel::Error:
		{
			UE_LOGF(LogAndroidStats, Error, "%s", Message);
			break;
		}
		case libGPUCountersLogLevel::Log:
		default:
		{
			UE_LOGF(LogAndroidStats, Log, "%s", Message);
			break;
		}
	}
}

static void InitGPUStats()
{
	UE_LOGF(LogAndroidStats, Log, "HWCPipe: GAndroidHWCPipeStatsEnabled=%d (set on commandline)", GAndroidHWCPipeStatsEnabled);
	if (!GAndroidHWCPipeStatsEnabled)
	{
		return;
	}

	libGPUCountersInit(GPUStatsLogCallback);
}

static void UpdateGPUStats()
{
	libGPUCountersUpdate();
}

static void LogGPUStats()
{
	libGPUCountersLog();
}

#else
static void InitGPUStats() {}
static void UpdateGPUStats() {}
static void LogGPUStats() {}
#endif
