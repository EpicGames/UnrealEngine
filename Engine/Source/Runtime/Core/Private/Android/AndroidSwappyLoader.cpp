// Copyright Epic Games, Inc. All Rights Reserved.

#include "AndroidSwappyLoader.h"

#undef VK_NO_PROTOTYPES
#include "swappy/swappyVk.h"
#include "swappy/swappyGL.h"
#include "swappy/swappyGL_extra.h"

#include <dlfcn.h>


#define APPLY_SWAPPY_FUNCTIONS(macro) \
	macro(SwappyVk_determineDeviceExtensions) \
	macro(SwappyVk_setQueueFamilyIndex) \
	macro(SwappyVk_initAndGetRefreshCycleDuration) \
	macro(SwappyVk_setWindow) \
	macro(SwappyVk_setSwapIntervalNS) \
	macro(SwappyVk_queuePresent) \
	macro(SwappyVk_destroySwapchain) \
	macro(SwappyVk_destroyDevice) \
	macro(SwappyVk_setAutoSwapInterval) \
	macro(SwappyVk_setAutoPipelineMode) \
	macro(SwappyVk_setMaxAutoSwapIntervalNS) \
	macro(SwappyVk_setFenceTimeoutNS) \
	macro(SwappyVk_getFenceTimeoutNS) \
	macro(SwappyVk_injectTracer) \
	macro(SwappyVk_setFunctionProvider) \
	macro(SwappyVk_getSwapIntervalNS) \
	macro(SwappyVk_getSupportedRefreshPeriodsNS) \
	macro(SwappyVk_isEnabled) \
	macro(SwappyVk_enableStats) \
	macro(SwappyVk_recordFrameStart) \
	macro(SwappyVk_getStats) \
	macro(SwappyVk_uninjectTracer) \
	macro(SwappyVk_clearStats) \
	macro(SwappyVk_resetFramePacing) \
	macro(SwappyVk_enableFramePacing) \
	macro(SwappyVk_enableBlockingWait) \
	macro(SwappyGL_init) \
	macro(SwappyGL_isEnabled) \
	macro(SwappyGL_destroy) \
	macro(SwappyGL_setWindow) \
	macro(SwappyGL_swap) \
	macro(SwappyGL_setUseAffinity) \
	macro(SwappyGL_setSwapIntervalNS) \
	macro(SwappyGL_setFenceTimeoutNS) \
	macro(SwappyGL_getRefreshPeriodNanos) \
	macro(SwappyGL_getSwapIntervalNS) \
	macro(SwappyGL_getUseAffinity) \
	macro(SwappyGL_getFenceTimeoutNS) \
	macro(SwappyGL_setBufferStuffingFixWait) \
	macro(SwappyGL_getSupportedRefreshPeriodsNS) \
	macro(SwappyGL_onChoreographer) \
	macro(SwappyGL_injectTracer) \
	macro(SwappyGL_setAutoSwapInterval) \
	macro(SwappyGL_setMaxAutoSwapIntervalNS) \
	macro(SwappyGL_setAutoPipelineMode) \
	macro(SwappyGL_enableStats) \
	macro(SwappyGL_recordFrameStart) \
	macro(SwappyGL_getStats) \
	macro(SwappyGL_uninjectTracer) \
	macro(SwappyGL_clearStats) \
	macro(SwappyGL_resetFramePacing) \
	macro(SwappyGL_enableFramePacing) \
	macro(SwappyGL_enableBlockingWait) \
	macro(Swappy_version) \
	macro(Swappy_setThreadFunctions) \
	macro(Swappy_versionString)

// Function pointer typedefs such as: typedef void (*FPTR_SwappyGL_destroy)();
#define DECLARE_SWAPPY_FPTR(func) typedef decltype(&func) FPTR_##func;
APPLY_SWAPPY_FUNCTIONS(DECLARE_SWAPPY_FPTR)
#undef DECLARE_SWAPPY_FPTR

// Function pointers such as: FPTR_SwappyGL_destroy fpSwappyGL_destroy;
#define DEFINE_SWAPPY_FPTR(func) FPTR_##func fp##func;
APPLY_SWAPPY_FUNCTIONS(DEFINE_SWAPPY_FPTR)
#undef DEFINE_SWAPPY_FPTR

static void* SwappyLibHandle;
void LoadSwappy()
{
	check(!SwappyLibHandle);
	SwappyLibHandle = dlopen("libswappy.so", RTLD_NOW | RTLD_LOCAL);
	if (!SwappyLibHandle)
	{
		UE_LOGF(LogAndroid, Log, "Failed to load libswappy.so");
		return;
	}

	// Load function pointers such as: fpSwappyGL_destroy = (FPTR_SwappyGL_destroy) dlsym(SwappyLibHandle, "SwappyGL_destroy");
	#define LOAD_SWAPPY_FPTR(func) \
			fp##func = (FPTR_##func) dlsym(SwappyLibHandle, #func); \
			if (!fp##func) { \
				UE_LOG(LogAndroid, Log, TEXT("Failed to load Swappy function " #func)); \
			}

	APPLY_SWAPPY_FUNCTIONS(LOAD_SWAPPY_FPTR)

	#undef LOAD_SWAPPY_FPTR
}

void UnloadSwappy()
{
	if (!SwappyLibHandle)
	{
		return;
	}

	dlclose(SwappyLibHandle);
	SwappyLibHandle = 0;

	#define ZERO_SWAPPY_FPTR(func) fp##func = 0;
	APPLY_SWAPPY_FUNCTIONS(ZERO_SWAPPY_FPTR)
	#undef ZERO_SWAPPY_FPTR
}


// Swappy function wrappers
uint32_t Swappy_version()
{
	return fpSwappy_version();
}

void Swappy_setThreadFunctions(const SwappyThreadFunctions* thread_functions)
{
	fpSwappy_setThreadFunctions(thread_functions);
}

const char* Swappy_versionString()
{
	return fpSwappy_versionString();
}

void SwappyVk_determineDeviceExtensions(
	VkPhysicalDevice physicalDevice, uint32_t availableExtensionCount,
	VkExtensionProperties* pAvailableExtensions,
	uint32_t* pRequiredExtensionCount, char** pRequiredExtensions)
{
	fpSwappyVk_determineDeviceExtensions(physicalDevice, availableExtensionCount, pAvailableExtensions, pRequiredExtensionCount, pRequiredExtensions);
}

void SwappyVk_setQueueFamilyIndex(VkDevice device, VkQueue queue,
	uint32_t queueFamilyIndex)
{
	fpSwappyVk_setQueueFamilyIndex(device, queue, queueFamilyIndex);
}

bool SwappyVk_initAndGetRefreshCycleDuration(JNIEnv* env, jobject jactivity,
	VkPhysicalDevice physicalDevice,
	VkDevice device,
	VkSwapchainKHR swapchain,
	uint64_t* pRefreshDuration)
{
	return fpSwappyVk_initAndGetRefreshCycleDuration(env, jactivity, physicalDevice, device, swapchain, pRefreshDuration);
}

void SwappyVk_setWindow(VkDevice device, VkSwapchainKHR swapchain,
	ANativeWindow* window)
{
	fpSwappyVk_setWindow(device, swapchain, window);
}

void SwappyVk_setSwapIntervalNS(VkDevice device, VkSwapchainKHR swapchain,
	uint64_t swap_ns)
{
	fpSwappyVk_setSwapIntervalNS(device, swapchain, swap_ns);
}

VkResult SwappyVk_queuePresent(VkQueue queue,
	const VkPresentInfoKHR* pPresentInfo)
{
	return fpSwappyVk_queuePresent(queue, pPresentInfo);
}

void SwappyVk_destroySwapchain(VkDevice device, VkSwapchainKHR swapchain)
{
	fpSwappyVk_destroySwapchain(device, swapchain);
}

void SwappyVk_destroyDevice(VkDevice device)
{
	fpSwappyVk_destroyDevice(device);
}

void SwappyVk_setAutoSwapInterval(bool enabled)
{
	fpSwappyVk_setAutoSwapInterval(enabled);
}

void SwappyVk_setAutoPipelineMode(bool enabled)
{
	fpSwappyVk_setAutoPipelineMode(enabled);
}

void SwappyVk_setMaxAutoSwapIntervalNS(uint64_t max_swap_ns)
{
	fpSwappyVk_setMaxAutoSwapIntervalNS(max_swap_ns);
}

void SwappyVk_setFenceTimeoutNS(uint64_t fence_timeout_ns)
{
	fpSwappyVk_setFenceTimeoutNS(fence_timeout_ns);
}

uint64_t SwappyVk_getFenceTimeoutNS()
{
	return fpSwappyVk_getFenceTimeoutNS();
}

void SwappyVk_injectTracer(const SwappyTracer* tracer)
{
	fpSwappyVk_injectTracer(tracer);
}

void SwappyVk_uninjectTracer(const SwappyTracer* tracer)
{
	fpSwappyVk_uninjectTracer(tracer);
}

void SwappyVk_setFunctionProvider(
	const SwappyVkFunctionProvider* pSwappyVkFunctionProvider)
{
	fpSwappyVk_setFunctionProvider(pSwappyVkFunctionProvider);
}

uint64_t SwappyVk_getSwapIntervalNS(VkSwapchainKHR swapchain)
{
	return fpSwappyVk_getSwapIntervalNS(swapchain);
}

int SwappyVk_getSupportedRefreshPeriodsNS(uint64_t* out_refreshrates,
	int allocated_entries,
	VkSwapchainKHR swapchain)
{
	return fpSwappyVk_getSupportedRefreshPeriodsNS(out_refreshrates, allocated_entries, swapchain);
}

bool SwappyVk_isEnabled(VkSwapchainKHR swapchain, bool* isEnabled)
{
	return fpSwappyVk_isEnabled(swapchain, isEnabled);
}

void SwappyVk_enableStats(VkSwapchainKHR swapchain, bool enabled)
{
	fpSwappyVk_enableStats(swapchain, enabled);
}

void SwappyVk_recordFrameStart(VkQueue queue, VkSwapchainKHR swapchain, uint32_t image)
{
	fpSwappyVk_recordFrameStart(queue, swapchain, image);
}

void SwappyVk_getStats(VkSwapchainKHR swapchain, SwappyStats* swappyStats)
{
	fpSwappyVk_getStats(swapchain, swappyStats);
}

void SwappyVk_clearStats(VkSwapchainKHR swapchain)
{
	fpSwappyVk_clearStats(swapchain);
}

void SwappyVk_resetFramePacing(VkSwapchainKHR swapchain)
{
	fpSwappyVk_resetFramePacing(swapchain);
}

void SwappyVk_enableFramePacing(VkSwapchainKHR swapchain, bool enable)
{
	fpSwappyVk_enableFramePacing(swapchain, enable);
}

void SwappyVk_enableBlockingWait(VkSwapchainKHR swapchain, bool enable)
{
	fpSwappyVk_enableBlockingWait(swapchain, enable);
}

bool SwappyGL_init(JNIEnv* env, jobject jactivity)
{
	return fpSwappyGL_init(env, jactivity);
}

bool SwappyGL_isEnabled()
{
	return fpSwappyGL_isEnabled();
}

void SwappyGL_destroy()
{
	fpSwappyGL_destroy();
}

bool SwappyGL_setWindow(ANativeWindow* window)
{
	return fpSwappyGL_setWindow(window);
}

bool SwappyGL_swap(EGLDisplay display, EGLSurface surface)
{
	return fpSwappyGL_swap(display, surface);
}

void SwappyGL_setUseAffinity(bool tf)
{
	fpSwappyGL_setUseAffinity(tf);
}

void SwappyGL_setSwapIntervalNS(uint64_t swap_ns)
{
	fpSwappyGL_setSwapIntervalNS(swap_ns);
}

void SwappyGL_setFenceTimeoutNS(uint64_t fence_timeout_ns)
{
	fpSwappyGL_setFenceTimeoutNS(fence_timeout_ns);
}

uint64_t SwappyGL_getRefreshPeriodNanos()
{
	return fpSwappyGL_getRefreshPeriodNanos();
}

uint64_t SwappyGL_getSwapIntervalNS()
{
	return fpSwappyGL_getSwapIntervalNS();
}

bool SwappyGL_getUseAffinity()
{
	return fpSwappyGL_getUseAffinity();
}

uint64_t SwappyGL_getFenceTimeoutNS()
{
	return fpSwappyGL_getFenceTimeoutNS();
}

void SwappyGL_setBufferStuffingFixWait(int32_t n_frames)
{
	fpSwappyGL_setBufferStuffingFixWait(n_frames);
}

int SwappyGL_getSupportedRefreshPeriodsNS(uint64_t* out_refreshrates, int allocated_entries)
{
	return fpSwappyGL_getSupportedRefreshPeriodsNS(out_refreshrates, allocated_entries);
}

void SwappyGL_onChoreographer(int64_t frameTimeNanos)
{
	fpSwappyGL_onChoreographer(frameTimeNanos);
}

void SwappyGL_injectTracer(const SwappyTracer* t)
{
	fpSwappyGL_injectTracer(t);
}

void SwappyGL_setAutoSwapInterval(bool enabled)
{
	fpSwappyGL_setAutoSwapInterval(enabled);
}

void SwappyGL_setMaxAutoSwapIntervalNS(uint64_t max_swap_ns)
{
	fpSwappyGL_setMaxAutoSwapIntervalNS(max_swap_ns);
}

void SwappyGL_setAutoPipelineMode(bool enabled)
{
	fpSwappyGL_setAutoPipelineMode(enabled);
}

void SwappyGL_enableStats(bool enabled)
{
	fpSwappyGL_enableStats(enabled);
}

void SwappyGL_recordFrameStart(EGLDisplay display, EGLSurface surface)
{
	fpSwappyGL_recordFrameStart(display, surface);
}

void SwappyGL_getStats(SwappyStats* swappyStats)
{
	fpSwappyGL_getStats(swappyStats);
}

void SwappyGL_uninjectTracer(const SwappyTracer* t)
{
	fpSwappyGL_uninjectTracer(t);
}

void SwappyGL_clearStats()
{
	fpSwappyGL_clearStats();
}

void SwappyGL_resetFramePacing()
{
	fpSwappyGL_resetFramePacing();
}

void SwappyGL_enableFramePacing(bool enable)
{
	fpSwappyGL_enableFramePacing(enable);
}

void SwappyGL_enableBlockingWait(bool enable)
{
	fpSwappyGL_enableBlockingWait(enable);
}

