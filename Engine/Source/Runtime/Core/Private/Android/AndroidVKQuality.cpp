// Copyright Epic Games, Inc. All Rights Reserved.

#include "AndroidVKQuality.h"

#if USE_ANDROID_JNI

#include "Android/AndroidJavaEnv.h"
#include <dlfcn.h>
#include "vkquality.h"

// entrypoints we want to load
#define ENUM_VKQUALITY_ENTRYPOINTS(EnumMacro) \
	EnumMacro(vkQuality_initializeFlagsInfo) \
	EnumMacro(vkQuality_destroy) \
	EnumMacro(vkQuality_getRecommendation)


// recommendations to recognize
#define ENUM_VKQUALITY_RECOMMENDATIONS(EnumMacro) \
	EnumMacro(VulkanBecauseDeviceMatch) \
	EnumMacro(VulkanBecausePredictionMatch) \
	EnumMacro(VulkanBecauseFutureAndroid) \
	EnumMacro(GLESBecauseOldDevice) \
	EnumMacro(GLESBecauseOldDriver) \
	EnumMacro(GLESBecauseNoDeviceMatch) \
	EnumMacro(GLESBecausePredictionMatch)


// declare types
#define ENUM_VKQUALITY_TYPE(f) using f##_t = decltype(&f);
ENUM_VKQUALITY_ENTRYPOINTS(ENUM_VKQUALITY_TYPE);

extern AAssetManager* AndroidThunkCpp_GetAssetManager();
extern FString GExternalFilePath;

namespace AndroidVKQuality
{
	void* VKQualityLib = nullptr;

	//  declare function pointers
	#define ENUM_VKQUALITY_FPTRS(f) f##_t f = nullptr;
	ENUM_VKQUALITY_ENTRYPOINTS(ENUM_VKQUALITY_FPTRS);

	bool LoadVkQuality(const char* GLESVersionString, void* VulkanPhysicalDeviceProperties)
	{
		// Load library
		VKQualityLib = dlopen("libvkquality.so", RTLD_NOW | RTLD_LOCAL);

		if (!VKQualityLib)
		{
			UE_LOGF(LogAndroid, Warning, "VKQuality: libvkquality.so could not be loaded");
			return false;
		}

		// Load each entrypoint
		bool bLoadedAllEntryPoints = true;
		#define ENUM_VKQUALITY_LOAD(f) f = (f##_t)dlsym(VKQualityLib, #f); bLoadedAllEntryPoints = bLoadedAllEntryPoints && (f != nullptr);
		ENUM_VKQUALITY_ENTRYPOINTS(ENUM_VKQUALITY_LOAD);

		if (!bLoadedAllEntryPoints)
		{
			UE_LOGF(LogAndroid, Warning, "VKQuality: not all entrypoints were loaded correctly");
			// unload .so
			UnloadVkQuality();
			return false;
		}

		vkqGraphicsAPIInfo APIInfo = 
			{
				GLESVersionString,
				VulkanPhysicalDeviceProperties
			};

		vkQualityInitResult Result = vkQuality_initializeFlagsInfo(AndroidJavaEnv::GetJavaEnv(), AndroidThunkCpp_GetAssetManager(),
			TCHAR_TO_UTF8(*GExternalFilePath),
			"vkqualitydata.vkq",
			&APIInfo,
			0);

		if (Result != kSuccess)
		{
			UE_LOGF(LogAndroid, Warning, "VKQuality: initialization failed with error %d", Result);		
			// unload .so
			UnloadVkQuality();
			return false;
		}

		return true;
	}

	void UnloadVkQuality()
	{
		check(VKQualityLib);

		dlclose(VKQualityLib);

		#define ENUM_VKQUALITY_NULLPTRS(f) f = nullptr;
		ENUM_VKQUALITY_ENTRYPOINTS(ENUM_VKQUALITY_NULLPTRS);

		VKQualityLib = nullptr;
	}

	FString GetVKQualityRecommendation()
	{
		check(VKQualityLib);

		vkQualityRecommendation Recommendation = vkQuality_getRecommendation();

		#define ENUM_VKQUALITY_RECOMMENDATION_CASE(r) \
			case kRecommendation##r: \
				return TEXT(#r);

		switch (Recommendation)
		{
			ENUM_VKQUALITY_RECOMMENDATIONS(ENUM_VKQUALITY_RECOMMENDATION_CASE);
			default:
				UE_LOGF(LogAndroid, Warning, "VKQuality: unrecognized recommendation %d", Recommendation);
				return TEXT("");
		}
	}

} // namespace

#endif