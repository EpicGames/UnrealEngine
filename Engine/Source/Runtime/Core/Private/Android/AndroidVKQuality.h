// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "CoreTypes.h"

#if USE_ANDROID_JNI

namespace AndroidVKQuality
{
	bool LoadVkQuality(const char* GLESVersionString, void* VulkanPhysicalDeviceProperties);
	void UnloadVkQuality();
	FString GetVKQualityRecommendation();
} // namespace

#endif