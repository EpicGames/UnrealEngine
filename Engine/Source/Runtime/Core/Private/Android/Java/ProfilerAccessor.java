// Copyright Epic Games, Inc. All Rights Reserved.
package com.epicgames.unreal.androidprofiling;

import android.content.Context;
import android.os.Build;
import android.os.ProfilingResult;
import java.util.HashMap;
import java.util.Map;
import java.util.Objects;
import java.util.function.Consumer;
import java.util.concurrent.Executors;
import com.epicgames.unreal.Logger;
import com.epicgames.unreal.GameActivity;
import androidx.annotation.RequiresApi;
import androidx.core.os.BufferFillPolicy;
import androidx.core.os.HeapProfileRequestBuilder;
import androidx.core.os.ProfilingRequest;
import androidx.core.os.StackSamplingRequestBuilder;
import androidx.core.os.SystemTraceRequestBuilder;

public class ProfilerAccessor
{
	public static void Init(GameActivity InActivity)
	{
		mActivity = InActivity;
	}
	private static GameActivity mActivity;

	public static Logger Log = new Logger("UE", "ProfilerAccessor");
	abstract static class BaseProfilerType
	{
		public String GetMode() {return "none";}

		public String InvokeProfiler(Context context, Map<String, String> Args, Consumer<android.os.ProfilingResult> listener) { return ""; }

		public ProfilingRequest BuildProfilerRequest(Map<String, String> Args, android.os.CancellationSignal cancellationSignal, StringBuilder OutputMessage) { return null; }

		public final static String ProfileSessionName = "profilename";

		public Map<String, String> GetDefaultArgs()
		{
			// args common to all profile types. 
			Map<String, String> DefaultArgs = new HashMap<String, String>();
			DefaultArgs.put("duration", "10");
			DefaultArgs.put("buffersize", "1000");
			DefaultArgs.put(ProfileSessionName, "profile");
			return DefaultArgs;
		}

		public String DefaultArgsHelp()
		{
			StringBuilder ret = new StringBuilder();
			for (Map.Entry<String, String> entry : GetDefaultArgs().entrySet())
			{
				ret.append(", [").append(entry.getKey()).append(" ").append(entry.getValue()).append("]");
			}
			return ret.toString();
		}
	};

	static class CallstackProfiler extends com.epicgames.unreal.androidprofiling.ProfilerAccessor.BaseProfilerType
	{
		@Override
		public String GetMode() {return "callstack";}

		@Override
		public Map<String, String> GetDefaultArgs()
		{
			Map<String, String> DefaultArgs = super.GetDefaultArgs();
			DefaultArgs.put("samplingfreq", "100");
			return DefaultArgs;
		}

		@RequiresApi(api = Build.VERSION_CODES.VANILLA_ICE_CREAM)
		public ProfilingRequest BuildProfilerRequest(Map<String, String> Args, android.os.CancellationSignal cancellationSignal, StringBuilder OutputMessage)
		{
			return new StackSamplingRequestBuilder()
					.setBufferSizeKb(Integer.parseInt(Objects.requireNonNull(Args.get("buffersize"))) /* Requested buffer size in KB */)
					.setDurationMs(Integer.parseInt(Args.get("duration")) * 1000 /* Requested profiling duration in millisconds */)
					.setSamplingFrequencyHz(Integer.parseInt(Args.get("samplingfreq")) /* Requested sampling frequency */)
					.setTag(Args.get(BaseProfilerType.ProfileSessionName) /* Caller supplied tag for identification */)
					.setCancellationSignal(cancellationSignal)
					.build();
		}
	}

	static class HeapProfiler extends com.epicgames.unreal.androidprofiling.ProfilerAccessor.BaseProfilerType
	{
		@Override
		public String GetMode() {return "heap";}

		@Override
		public Map<String, String> GetDefaultArgs()
		{
			Map<String, String> DefaultArgs = super.GetDefaultArgs();
			DefaultArgs.put("samplingintervalbytes", "100");
			return DefaultArgs;
		}

		@RequiresApi(api = Build.VERSION_CODES.VANILLA_ICE_CREAM)
		public ProfilingRequest BuildProfilerRequest(Map<String, String> Args, android.os.CancellationSignal cancellationSignal, StringBuilder OutputMessage)
		{
			return new HeapProfileRequestBuilder()
					.setBufferSizeKb(Integer.parseInt(Objects.requireNonNull(Args.get("buffersize"))) /* Requested buffer size in KB */)
					.setDurationMs(Integer.parseInt(Args.get("duration")) * 1000 /* Requested profiling duration in millisconds */)
					.setTrackJavaAllocations(false)
					.setSamplingIntervalBytes(Integer.parseInt(Args.get("samplingintervalbytes")) /* Requested sampling interval in bytes */)
					.setTag(Args.get(BaseProfilerType.ProfileSessionName) /* Caller supplied tag for identification */)
					.setCancellationSignal(cancellationSignal)
					.build();
		}
	}

	static class SystemProfiler extends com.epicgames.unreal.androidprofiling.ProfilerAccessor.BaseProfilerType
	{
		@Override
		public String GetMode() {return "system";}

		@Override
		public Map<String, String> GetDefaultArgs()
		{
			Map<String, String> DefaultArgs = super.GetDefaultArgs();
			DefaultArgs.put("bufferfillpolicy", "ringbuffer");
			return DefaultArgs;
		}
		@RequiresApi(api = Build.VERSION_CODES.VANILLA_ICE_CREAM)
		public ProfilingRequest BuildProfilerRequest(Map<String, String> Args, android.os.CancellationSignal cancellationSignal, StringBuilder OutputMessage)
		{
			BufferFillPolicy fillPolicy;
			String desiredFillPolicy = Args.get("bufferfillpolicy");
			switch( desiredFillPolicy )
			{
				case "ringbuffer":
					fillPolicy = BufferFillPolicy.RING_BUFFER;
					break;
				case "discard":
					fillPolicy = BufferFillPolicy.DISCARD;
					break;
				default:
					OutputMessage.append("Ignoring unknown bufferfillpolicy ("+desiredFillPolicy+")");
					Log.error(OutputMessage.toString());
					fillPolicy = BufferFillPolicy.RING_BUFFER;
					break;
			}

			return new SystemTraceRequestBuilder()
					.setBufferSizeKb(Integer.parseInt(Args.get("buffersize")) /* Requested buffer size in KB */)
					.setDurationMs(Integer.parseInt(Args.get("duration")) * 1000 /* Requested profiling duration in millisconds */)
					.setBufferFillPolicy(fillPolicy /* Buffer fill policy */)
					.setTag(Args.get(BaseProfilerType.ProfileSessionName) /* Caller supplied tag for identification */)
					.setCancellationSignal(cancellationSignal)
					.build();
		}

	}

	static com.epicgames.unreal.androidprofiling.ProfilerAccessor.BaseProfilerType[] AllProfilers = {
			new com.epicgames.unreal.androidprofiling.ProfilerAccessor.CallstackProfiler(),
			new com.epicgames.unreal.androidprofiling.ProfilerAccessor.SystemProfiler(),
			new com.epicgames.unreal.androidprofiling.ProfilerAccessor.HeapProfiler()
	};


	private static String[] OnError(String ErrorMessage)
	{
		return new String[] { null, ErrorMessage};
	}

	private static String[] OnSessionIssued(String Tag, String Message)
	{
		return new String[] { Tag, Message};
	}

	private static native void nativeOnProfileFinish(String Tag, String Mesage, String FilePath);
	static class ActiveProfileSession
	{
		public android.os.CancellationSignal CancelSignal;
	};

	private static final Map<String, ActiveProfileSession> ActiveSessions = new HashMap<String,ActiveProfileSession>();

	private static com.epicgames.unreal.androidprofiling.ProfilerAccessor.BaseProfilerType FindProfiler(String RequestedProfilerMode)
	{
		for ( com.epicgames.unreal.androidprofiling.ProfilerAccessor.BaseProfilerType Profiler : AllProfilers)
		{
			if( Profiler.GetMode().equals(RequestedProfilerMode) )
			{
				return Profiler;
			}
		}
		return null;
	}
	
	// returns String[] = { profile tag (or null on error), message }
	public synchronized static String[] IssueProfilerCommand(String RequestedProfilerArgs)
	{
		if(Build.VERSION.SDK_INT<Build.VERSION_CODES.VANILLA_ICE_CREAM) // Build.VERSION.VANILLA_ICE_CREAM == 35
		{
			return OnError("API version must be >= "+Build.VERSION_CODES.VANILLA_ICE_CREAM+". (current API version = "+ Build.VERSION.SDK_INT+")");
		}

		int FirstCommaIdx = RequestedProfilerArgs.indexOf(" ");
		String RequestedProfilerMode = (FirstCommaIdx == -1 ? RequestedProfilerArgs.trim() : RequestedProfilerArgs.substring(0, FirstCommaIdx)).toLowerCase();
		com.epicgames.unreal.androidprofiling.ProfilerAccessor.BaseProfilerType FoundProfiler = FindProfiler(RequestedProfilerMode);

		if( FoundProfiler == null )
		{
			StringBuilder message = new StringBuilder("usage:\nandroid.profiler <profiler mode> [, optional comma separated args]\n\n");
			for ( com.epicgames.unreal.androidprofiling.ProfilerAccessor.BaseProfilerType Profiler : AllProfilers)
			{
				message.append("\tandroid.profiler "+Profiler.GetMode()+Profiler.DefaultArgsHelp()+"\n");
			}
			return OnError(message.toString());
		}

		// find all comma separated args
		Map<String, String> ArgsMap = FoundProfiler.GetDefaultArgs();
		for(String CombinedArg : RequestedProfilerArgs.split(" "))
		{
			// split the space separated args by = to get arg and parameter. 
			String Arg = CombinedArg, Parameter = "";
			int FirstIdx = CombinedArg.indexOf("=");
			if( FirstIdx > 0 )
			{
				Arg = CombinedArg.substring(0, FirstIdx);
				Parameter = CombinedArg.substring(FirstIdx+1);
			}
			ArgsMap.put(Arg, Parameter);
		}

		String sessionName = ArgsMap.get(BaseProfilerType.ProfileSessionName);
		// profile tag must be alphanum and <=20 chars.
		sessionName = sessionName.replaceAll("[^0-9a-zA-Z]", "");
		sessionName = sessionName.substring(0, Math.min(sessionName.length(), 20));
		
		ActiveProfileSession newSession = new ActiveProfileSession();
			
		if( ActiveSessions.putIfAbsent(sessionName, newSession ) != null) 
		{
			// name in use.
			return OnError("Profile session name '"+sessionName+"' is in use.");
		}

		Consumer<android.os.ProfilingResult> listener = profilingResult ->
		{
			OnSessionComplete(profilingResult);
		};

		StringBuilder InvocationMessage = new StringBuilder();
		try {
			java.util.concurrent.Executor executor = Executors.newSingleThreadExecutor();
			newSession.CancelSignal = new android.os.CancellationSignal();
			ProfilingRequest Request = FoundProfiler.BuildProfilerRequest(ArgsMap, newSession.CancelSignal, InvocationMessage);

			androidx.core.os.Profiling.requestProfiling(
					mActivity.getApplicationContext(),
					Request,
					executor,
					listener
			);
		}
		catch (Exception e)
		{
			if(!InvocationMessage.isEmpty())
			{
				InvocationMessage.append("\n");
			}
			InvocationMessage.append("profile session failed: "+e.toString()+" cause: "+ e.getCause());
			return OnError(InvocationMessage.toString());
		}
		return OnSessionIssued(sessionName, InvocationMessage.toString());
	}

	@RequiresApi(api = Build.VERSION_CODES.VANILLA_ICE_CREAM)
	private static synchronized void OnSessionComplete(ProfilingResult profilingResult) 
	{
		if (profilingResult.getErrorCode() == ProfilingResult.ERROR_NONE)
		{
			Log.debug("profile success.. (retrieve with 'adb pull "+profilingResult.getResultFilePath()+"')");
		}
		else
		{
			Log.error("profile failed: ("+profilingResult.getErrorCode()+") " +profilingResult.getErrorMessage());
		}

		String resultSessionName = profilingResult.getTag();
		String Message = "("+profilingResult.getErrorCode()+") " +profilingResult.getErrorMessage();
		String FilePath = profilingResult.getResultFilePath();

		nativeOnProfileFinish(resultSessionName, Message, FilePath);

		ActiveSessions.remove(resultSessionName);
	}

	// returns true if a session was in progress and was cancelled
	public static synchronized boolean CancelProfile(String sessionName)
	{
		ActiveProfileSession foundSession = ActiveSessions.get(sessionName);
		if( foundSession != null)
		{
			Log.debug("profile: ("+sessionName+") was cancelled.");
			foundSession.CancelSignal.cancel();
			return true;
		}
		Log.error("could not cancel profile: ("+sessionName+") was not found.");
		return false;
	}

	public static String[] AndroidThunkJava_IssueProfilerCommand(String RequestedProfilerArgs)
	{
		return IssueProfilerCommand(RequestedProfilerArgs);
	}

	public static boolean AndroidThunkJava_StopProfilerCommand(String profileSessionName)
	{
		return CancelProfile(profileSessionName);
	}
}