// Copyright Epic Games, Inc. All Rights Reserved.

#include "Apple/AppleAsyncTask.h"
#include "HAL/PlatformAtomics.h"

@implementation FAppleAsyncTask

@synthesize UserData;
@synthesize GameThreadCallback;

/** All currently running tasks (which can be created on iOS thread or main thread) */
NSMutableArray* RunningTasks;

/**
 * Static class constructor, called before any instances are created
 */
+ (void)initialize
{
	// create the RunningTasks object one time
	RunningTasks = [[NSMutableArray arrayWithCapacity:4] retain];
}

/**
 * Initialize the async task
 */
- (id)init
{
	self = [super init];

	// add ourself to the list of tasks
	@synchronized(RunningTasks)
	{
		[RunningTasks addObject:self];
	}

	// return ourself, the constructed object
	return self;
}


+ (void)CreateTaskWithBlock:(bool (^)(void))Block
{
	// create a task, and add it to the array
	FAppleAsyncTask* Task = [[FAppleAsyncTask alloc] init];
	// set the callback
	Task.GameThreadCallback = Block;
	// safely tell the game thread we are ready to go
	[Task FinishedTask];
}

- (void)FinishedTask
{
	FPlatformAtomics::InterlockedIncrement(&bIsReadyForGameThread);
}

/**
 * Check for completion
 *
 * @return TRUE if we succeeded (the completion block will have been called)
 */
- (bool)CheckForCompletion
{
	// handle completion
	if (bIsReadyForGameThread)
	{
		// call the game thread block
		if (GameThreadCallback)
		{
			if (GameThreadCallback())
			{
				// only return true if the callback says it's complete
				return true;
			}
		}
		else
		{
			// if there isn't a callback, then just return TRUE to remove the 
			// task from the queue
			return true;
		}
		
	}

	// all other cases, we are not complete
	return false;
}

/**
 * Tick all currently running tasks
 */
+ (void)ProcessAsyncTasks
{
	while (true)
	{
		FAppleAsyncTask* CurrentTask = nil;

		// Grab the first task from the queue
		@synchronized(RunningTasks)
		{
			if ([RunningTasks count] > 0)
			{
				CurrentTask = [RunningTasks objectAtIndex:0];
			}
		}

		if (CurrentTask == nil)
		{
			break;
		}

		// Check and run the task
		if (![CurrentTask CheckForCompletion])
		{
			// Task not ready yet, break and resume on next frame to preserve task order
			break;
		}

		// Success, remove the task and release it
		@synchronized(RunningTasks)
		{
			[RunningTasks removeObjectAtIndex:0];
		}
		[CurrentTask release];
	}
}


/** 
 * Application destructor
 */
- (void)dealloc 
{
	[GameThreadCallback release];
	self.UserData = nil;

	[super dealloc];
}


@end
