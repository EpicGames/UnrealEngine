// Copyright Epic Games, Inc. All Rights Reserved.

#include "Apple/AppleContactManager.h"

#if !PLATFORM_TVOS && WITH_APPLE_CONTACTS

DEFINE_LOG_CATEGORY_STATIC(LogAppleContactManager, Log, All);

FAppleContactManager::FAppleContactManager()
{
	ContactStore = [[CNContactStore alloc] init];
}

bool FAppleContactManager::HaveContactAccessAuthorization()
{
	CNAuthorizationStatus AuthStatus = [CNContactStore authorizationStatusForEntityType: CNEntityTypeContacts];
	
#if PLATFORM_MAC
	return AuthStatus == CNAuthorizationStatusAuthorized;
#else
	return AuthStatus == CNAuthorizationStatusLimited || AuthStatus == CNAuthorizationStatusAuthorized;
#endif
}

void FAppleContactManager::RequestContactAccess()
{
	[ContactStore requestAccessForEntityType:CNEntityTypeContacts completionHandler:^(BOOL Granted, NSError* _Nullable Error) 
	 {
		if(!Granted)
		{
			const char* DomainCString = [Error.domain UTF8String];
			
			UE_LOGF(LogAppleContactManager, Error, "Couldn't get access to contacts due to NSError %s:%lld", DomainCString, Error.code)
		}
		
		OnContactsPermissionChanged.Broadcast(static_cast<bool>(Granted));       
	}];
}

FAppleContactManager::ELabeledContactType FAppleContactManager::GetContactTypeForString(NSString* Label, bool bPhoneNumber)
{
	if([Label isEqualToString: CNLabelHome])
	{
		return ELabeledContactType::Home;
	}
	else if([Label isEqualToString: CNLabelWork])
	{
		return ELabeledContactType::Work;
	}
	else if(bPhoneNumber && ([Label isEqualToString: CNLabelPhoneNumberMobile] || [Label isEqualToString: CNLabelPhoneNumberiPhone]))
	{
		return ELabeledContactType::Mobile;
	}
	else
	{
		return ELabeledContactType::Other;
	}
}

// This will fail silently or crash if permissions were not granted AND/OR
// you don't have a valid NSContactsUsageDescription string in your plist.
TArray<FAppleContactManager::FAppleContact> FAppleContactManager::GetAllContacts()
{
	if(!HaveContactAccessAuthorization())
	{
		UE_LOGF(LogAppleContactManager, Warning, 
			   "HaveContactAccessAuthorization() returned false and thus GetAllContacts will likely fail. Did you call RequestContactAccess before GetAllContacts? Do you have a NSContactsUsageDescription string in your plist?");
	}
	
	// Change keys for which contact details to grab here
	NSArray* Keys = @[CNContactFamilyNameKey, CNContactGivenNameKey, CNContactNicknameKey, 
					  CNContactEmailAddressesKey, CNContactPhoneNumbersKey];
	
	
	NSPredicate* Predicate = [NSPredicate predicateWithValue: true];
	NSError* Error = nil;
	
	NSArray* CNContacts = [ContactStore unifiedContactsMatchingPredicate:Predicate keysToFetch:Keys error:&Error];
	TArray<FAppleContact> ReturnValue;
	
	if(CNContacts) 
	{
		for (CNContact* Contact in CNContacts) 
		{
			FAppleContact AppleContact 
			{
				.FamilyName = Contact.familyName,
				.GivenName = Contact.givenName,
				.NickName = Contact.nickname
			};

			for (CNLabeledValue* Label in Contact.phoneNumbers) 
			{
				NSString* Phone = [Label.value stringValue];
				if ([Phone length] > 0) 
				{					
					FLabeledContactValue LabeledPhone
					{
						.Type = GetContactTypeForString(Label.label, true),
						.Value = Phone
					};
					
					AppleContact.PhoneNumbers.Add(LabeledPhone);
				}
			}
			
			for (CNLabeledValue* Label in Contact.emailAddresses) 
			{
				NSString* EmailAddress = [Label.value stringValue];
				if ([EmailAddress length] > 0) 
				{					
					FLabeledContactValue LabeledEmail 
					{
						.Type = GetContactTypeForString(Label.label, false),
						.Value = EmailAddress
					};
					
					AppleContact.EmailAddresses.Add(LabeledEmail);
				}
			}
			
			ReturnValue.Add(AppleContact);
		}
	} 
	else if (Error) 
	{
		const char* ErrorDomain = [Error.domain UTF8String];
		
		UE_LOGF(LogAppleContactManager, Error, 
			   "Couldn't fetch contacts! NSError domain %s, code %lld", ErrorDomain, Error.code);
	}
	
	return ReturnValue;
}

#endif
