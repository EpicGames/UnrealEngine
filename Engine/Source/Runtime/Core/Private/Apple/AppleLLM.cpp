// Copyright Epic Games, Inc. All Rights Reserved.

#include "Apple/AppleLLM.h"
#include "HAL/LowLevelMemStats.h"

#if ENABLE_LOW_LEVEL_MEM_TRACKER

#if PLATFORM_MAC
#include "Mac/MacSystemIncludes.h"
#endif

#include <objc/runtime.h>
#import <Foundation/Foundation.h>

struct FLLMTagInfoApple
{
	const TCHAR* Name;
	FName StatName;				// shows in the LLMFULL stat group
	FName SummaryStatName;		// shows in the LLM summary stat group
};

DECLARE_LLM_MEMORY_STAT(TEXT("Objective-C"), STAT_ObjectiveCLLM, STATGROUP_LLMPlatform);

// *** order must match ELLMTagApple enum ***
const FLLMTagInfoApple ELLMTagNamesApple[] =
{
	// csv name						// stat name								// summary stat name						// enum value
	{ TEXT("Objective-C"),				GET_STATFNAME(STAT_ObjectiveCLLM),		NAME_None },									// ELLMTagApple::ObjectiveC
};

typedef id (*AllocWithZoneIMP)(id Obj, SEL Sel, struct _NSZone* Zone);
typedef id (*DeallocIMP)(id Obj, SEL Sel);
static AllocWithZoneIMP AllocWithZoneOriginal = nullptr;
static DeallocIMP DeallocOriginal = nullptr;

static id AllocWithZoneInterposer(id Obj, SEL Sel, struct _NSZone * Zone)
{
	id Result = AllocWithZoneOriginal(Obj, Sel, Zone);
	
	if (Result)
	{
		LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelAlloc(ELLMTracker::Default, Result, class_getInstanceSize([Result class]), (ELLMTag)ELLMTagApple::ObjectiveC, ELLMAllocType::System));
	}
	
	return Result;
}

static void DeallocInterposer(id Obj, SEL Sel)
{
	if (Obj)
	{
		LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelFree(ELLMTracker::Default, Obj, ELLMAllocType::System));
	}
	DeallocOriginal(Obj, Sel);
}

static id (*NSAllocateObjectPtr)(Class aClass, NSUInteger extraBytes, NSZone *zone) = nullptr;

id NSAllocateObjectPtrInterposer(Class aClass, NSUInteger extraBytes, NSZone *zone)
{
	id Result = NSAllocateObjectPtr(aClass, extraBytes, zone);
	if (Result)
	{
		LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelAlloc(ELLMTracker::Default, Result, class_getInstanceSize([Result class]), (ELLMTag)ELLMTagApple::ObjectiveC, ELLMAllocType::System));
	}
	
	return Result;
}

static id (*_os_object_alloc_realizedPtr)(Class aClass, size_t size) = nullptr;
id _os_object_alloc_realizedInterposer(Class aClass, size_t size)
{
	id Result = _os_object_alloc_realizedPtr(aClass, size);
	if (Result)
	{
		LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelAlloc(ELLMTracker::Default, Result, class_getInstanceSize([Result class]), (ELLMTag)ELLMTagApple::ObjectiveC, ELLMAllocType::System));
	}
	
	return Result;
}

static void (*NSDeallocateObjectPtr)(id Obj) = nullptr;

void NSDeallocateObjectInterposer(id Obj)
{
	if (Obj)
	{
		LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelFree(ELLMTracker::Default, Obj, ELLMAllocType::System));
	}
	NSDeallocateObjectPtr(Obj);
}

/*
 * Register Apple tags with LLM and setup the NSObject +alloc:, -dealloc interposers, some Objective-C allocations will already have been made, but there's not much I can do about that.
 */
void AppleLLM::Initialise()
{
	int32 TagCount = sizeof(ELLMTagNamesApple) / sizeof(FLLMTagInfoApple);

	for (int32 Index = 0; Index < TagCount; ++Index)
	{
		int32 Tag = (int32)ELLMTag::PlatformTagStart + Index;
		const FLLMTagInfoApple& TagInfo = ELLMTagNamesApple[Index];

		FLowLevelMemTracker::Get().RegisterPlatformTag(Tag, TagInfo.Name, TagInfo.StatName, TagInfo.SummaryStatName);
	}
}

#endif		// #if ENABLE_LOW_LEVEL_MEM_TRACKER

