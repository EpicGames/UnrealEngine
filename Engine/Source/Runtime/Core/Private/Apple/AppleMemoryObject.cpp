// Copyright Epic Games, Inc. All Rights Reserved.

#include "Apple/AppleMemoryObject.h"

#include <stdlib.h>
#include <sys/param.h>
#include <sys/mman.h>
#include <sys/mount.h>
#include <objc/runtime.h>

#include "CoreTypes.h"
#if PLATFORM_MAC
#include <libproc.h>
#endif
#if PLATFORM_IOS
#include "IOS/IOSPlatformMisc.h"
#include <os/proc.h>
#endif
#include <CoreFoundation/CFBase.h>
#include <mach/vm_map.h>
#include <mach/vm_region.h>
#include <mach/vm_statistics.h>
#include <mach/vm_page_size.h>
#include "HAL/LowLevelMemTracker.h"
#include "Apple/AppleLLM.h"

#if PLATFORM_IOS
#include <sys/resource.h> // On iOS, iPadOS and tvOS.
extern "C" int proc_pid_rusage(int pid, int flavor, rusage_info_t *buffer)  
	__OSX_AVAILABLE_STARTING(__MAC_10_9, __IPHONE_7_0);
#endif

#include "HAL/UnrealMemory.h"
#include "HAL/PlatformMath.h"
#include "Templates/AlignmentTemplates.h"

NS_ASSUME_NONNULL_BEGIN

/** 
 * Zombie object implementation so that we can implement NSZombie behaviour for our custom allocated objects.
 * Will leak memory - just like Cocoa's NSZombie - but allows for debugging of invalid usage of the pooled types.
 */
@interface FApplePlatformObjectZombie : NSObject 
{
	@public
	Class OriginalClass;
}
@end

@implementation FApplePlatformObjectZombie
-(id)init
{
	self = (FApplePlatformObjectZombie*)[super init];
	if (self)
	{
		OriginalClass = nil;
	}
	return self;
}

-(void)dealloc
{
	// Denied!
	return;

	PRAGMA_DISABLE_UNREACHABLE_CODE_WARNINGS	
	[super dealloc];
	PRAGMA_RESTORE_UNREACHABLE_CODE_WARNINGS	
}

- (nullable NSMethodSignature *)methodSignatureForSelector:(SEL)sel
{
	NSLog(@"Selector %@ sent to deallocated instance %p of class %@", NSStringFromSelector(sel), self, OriginalClass);
	abort();
}
@end

@implementation FApplePlatformObject

+ (nullable OSQueueHead*)classAllocator
{
	return nullptr;
}

+ (id)allocClass: (Class)NewClass
{
	static bool NSZombieEnabled = (getenv("NSZombieEnabled") != nullptr);
	
	// Allocate the correct size & zero it
	// All allocations must be 16 byte aligned
	SIZE_T Size = Align(FPlatformMath::Max(class_getInstanceSize(NewClass), class_getInstanceSize([FApplePlatformObjectZombie class])), 16);
	void* Mem = nullptr;
	
	OSQueueHead* Alloc = [NewClass classAllocator];
	if (Alloc && !NSZombieEnabled)
	{
		Mem = OSAtomicDequeue(Alloc, 0);
		if (!Mem)
		{
			static uint8 BlocksPerChunk = 32;
			char* Chunk = (char*)FMemory::Malloc(Size * BlocksPerChunk);
			Mem = Chunk;
			Chunk += Size;
			for (uint8 i = 0; i < (BlocksPerChunk - 1); i++, Chunk += Size)
			{
				OSAtomicEnqueue(Alloc, Chunk, 0);
			}
		}
	}
	else
	{
		Mem = FMemory::Malloc(Size);
	}
	FMemory::Memzero(Mem, Size);
	
	// Construction assumes & requires zero-initialised memory
	FApplePlatformObject* Obj = (FApplePlatformObject*)objc_constructInstance(NewClass, Mem);
	object_setClass(Obj, NewClass);
	Obj->AllocatorPtr = !NSZombieEnabled ? Alloc : nullptr;
	return Obj;
}

- (void)dealloc
{
	static bool NSZombieEnabled = (getenv("NSZombieEnabled") != nullptr);
	
	// First call the destructor and then release the memory - like C++ placement new/delete
	objc_destructInstance(self);
	if (AllocatorPtr)
	{
		check(!NSZombieEnabled);
		OSAtomicEnqueue(AllocatorPtr, self, 0);
	}
	else if (NSZombieEnabled)
	{
		Class CurrentClass = self.class;
		object_setClass(self, [FApplePlatformObjectZombie class]);
		FApplePlatformObjectZombie* ZombieSelf = (FApplePlatformObjectZombie*)self;
		ZombieSelf->OriginalClass = CurrentClass;
	}
	else
	{
		FMemory::Free(self);
	}
	return;
	
	PRAGMA_DISABLE_UNREACHABLE_CODE_WARNINGS	
	// Deliberately unreachable code to silence clang's error about not calling super - which in all other
	// cases will be correct.
	[super dealloc];
	PRAGMA_RESTORE_UNREACHABLE_CODE_WARNINGS	
}

@end

NS_ASSUME_NONNULL_END
