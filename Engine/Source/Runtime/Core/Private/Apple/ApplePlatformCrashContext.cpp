// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ApplePlatformCrashContext.cpp: Common implementations of Apple platform crash context.
=============================================================================*/

#include "Apple/ApplePlatformCrashContext.h"
#include "Apple/ApplePlatformSymbolication.h"
#include "Containers/StringConv.h"
#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "CoreGlobals.h"
#include "Apple/AppleStringUtils.h"

THIRD_PARTY_INCLUDES_START
#if !PLATFORM_TVOS && !PLATFORM_VISIONOS
#include "PLCrashReporter.h"
#include "PLCrashReport.h"
#include "PLCrashReportTextFormatter.h"
#endif
THIRD_PARTY_INCLUDES_END

/** Implement platform specific static cleanup function */
void FGenericCrashContext::CleanupPlatformSpecificFiles()
{
	// Is this the correct way to cleanup on Apple?
	FString CrashVideoPath = FPaths::ProjectLogDir() + TEXT("CrashVideo.avi");
	IFileManager::Get().Delete(*CrashVideoPath);
}

FApplePlatformCrashContext::FApplePlatformCrashContext(ECrashContextType InType, const TCHAR* InErrorMessage)
:	FGenericCrashContext(InType, InErrorMessage)
,	Signal(0)
,	Info(nullptr)
,	Context(nullptr)
{
	SignalDescription[ 0 ] = 0;
	MinidumpCallstackInfo[ 0 ] = 0;
	CallstackProperty[ 0 ] = 0;
}

FApplePlatformCrashContext::~FApplePlatformCrashContext()
{
	
}

void FApplePlatformCrashContext::InitFromSignal(int32 InSignal, siginfo_t* InInfo, void* InContext)
{
	Signal = InSignal;
	Info = InInfo;
	ErrorFrame = InInfo->si_addr;
	Context = reinterpret_cast< ucontext_t* >( InContext );
	
#define HANDLE_CASE(a,b) case a: FCStringAnsi::Strncpy(SignalDescription, #a ": " b, UE_ARRAY_COUNT( SignalDescription ) - 1); break;
	switch (Signal)
	{
		case SIGSEGV:
			FCStringAnsi::Strncpy(SignalDescription, "SIGSEGV: invalid attempt to access memory at address 0x", UE_ARRAY_COUNT( SignalDescription ) - 1);
			FCStringAnsi::Strcat(SignalDescription, ItoANSI((uintptr_t)Info->si_addr, 16));
			break;
		case SIGBUS:
			FCStringAnsi::Strncpy(SignalDescription, "SIGBUS: invalid attempt to access memory at address 0x", UE_ARRAY_COUNT( SignalDescription ) - 1);
			FCStringAnsi::Strcat(SignalDescription, ItoANSI((uintptr_t)Info->si_addr, 16));
			break;
			HANDLE_CASE(SIGINT, "program interrupted")
			HANDLE_CASE(SIGQUIT, "user-requested crash")
			HANDLE_CASE(SIGILL, "illegal instruction")
			HANDLE_CASE(SIGTRAP, "trace trap")
			HANDLE_CASE(SIGABRT, "abort() called")
			HANDLE_CASE(SIGFPE, "floating-point exception")
			HANDLE_CASE(SIGKILL, "program killed")
			HANDLE_CASE(SIGSYS, "non-existent system call invoked")
			HANDLE_CASE(SIGPIPE, "write on a pipe with no reader")
			HANDLE_CASE(SIGTERM, "software termination signal")
			HANDLE_CASE(SIGSTOP, "stop")
			
		default:
			FCStringAnsi::Strncpy(SignalDescription, "Signal ", UE_ARRAY_COUNT( SignalDescription ) - 1);
			FCStringAnsi::Strcat(SignalDescription, ItoANSI(Signal, 10));
			FCStringAnsi::Strcat(SignalDescription, " (unknown)");
			break;
	}
#undef HANDLE_CASE
}

int32 FApplePlatformCrashContext::ReportCrash() const
{
	static bool GAlreadyCreatedMinidump = false;
	// Only create a minidump the first time this function is called.
	// (Can be called the first time from the RenderThread, then a second time from the MainThread.)
	if ( GAlreadyCreatedMinidump == false )
	{
		GAlreadyCreatedMinidump = true;
		
		ANSICHAR* StackBuffer = const_cast<ANSICHAR*>(&MinidumpCallstackInfo[0]);
		*StackBuffer = 0;
		
		// Generate the portable callstack for the crash xml
		const_cast<FApplePlatformCrashContext*>(this)->CapturePortableCallStack(ErrorFrame, Context);
		// Walk the stack and dump it to the allocated memory (ignore first 2 callstack lines as those are in stack walking code)
		FPlatformStackWalk::StackWalkAndDump( StackBuffer, UE_ARRAY_COUNT(MinidumpCallstackInfo) - 1, ErrorFrame, Context );
		// Escape it and put into CallstackProperty, to be used as <Callstack> in crash report
		FString EscapedStack;
		AppendEscapedXMLString(EscapedStack, FString(MinidumpCallstackInfo));
		FCString::Strncpy(const_cast<TCHAR*>(CallstackProperty), *EscapedStack, UE_ARRAY_COUNT(CallstackProperty));
		
		FUTF8ToTCHAR_Convert::Convert(GErrorHist, UE_ARRAY_COUNT(GErrorHist) - 1, MinidumpCallstackInfo, FCStringAnsi::Strlen(MinidumpCallstackInfo));
		CreateExceptionInfoString(Signal, Info);
	}
	
	return 0;
}

void FApplePlatformCrashContext::CreateExceptionInfoString(int32 Signal, siginfo_t* Info)
{
#define HANDLE_CASE(a,b) case a: FCString::Strncpy(GErrorExceptionDescription, TEXT(#a ": " b), UE_ARRAY_COUNT( SignalDescription ) - 1); break;
	switch (Signal)
	{
		case SIGSEGV:
			FCString::Strncpy(GErrorExceptionDescription, TEXT("SIGSEGV: invalid attempt to access memory at address 0x"), UE_ARRAY_COUNT( SignalDescription ) - 1);
			FCString::Strcat(GErrorExceptionDescription, ItoTCHAR((uintptr_t)Info->si_addr, 16));
			break;
		case SIGBUS:
			FCString::Strncpy(GErrorExceptionDescription, TEXT("SIGBUS: invalid attempt to access memory at address 0x"), UE_ARRAY_COUNT( SignalDescription ) - 1);
			FCString::Strcat(GErrorExceptionDescription, ItoTCHAR((uintptr_t)Info->si_addr, 16));
			break;
			HANDLE_CASE(SIGINT, "program interrupted")
			HANDLE_CASE(SIGQUIT, "user-requested crash")
			HANDLE_CASE(SIGILL, "illegal instruction")
			HANDLE_CASE(SIGTRAP, "trace trap")
			HANDLE_CASE(SIGABRT, "abort() called")
			HANDLE_CASE(SIGFPE, "floating-point exception")
			HANDLE_CASE(SIGKILL, "program killed")
			HANDLE_CASE(SIGSYS, "non-existent system call invoked")
			HANDLE_CASE(SIGPIPE, "write on a pipe with no reader")
			HANDLE_CASE(SIGTERM, "software termination signal")
			HANDLE_CASE(SIGSTOP, "stop")
			
		default:
			FCString::Strncpy(GErrorExceptionDescription, TEXT("Signal "), UE_ARRAY_COUNT( SignalDescription ) - 1);
			FCString::Strcat(GErrorExceptionDescription, ItoTCHAR(Signal, 10));
			FCString::Strcat(GErrorExceptionDescription, TEXT(" (unknown)"));
			break;
	}
#undef HANDLE_CASE
}

void FApplePlatformCrashContext::ConvertMinidump(char const* OutputPath, char const* InputPath)
{
#if !PLATFORM_TVOS && !PLATFORM_VISIONOS
	NSError* Error = nil;
	NSString* Path = FString(ANSI_TO_TCHAR(InputPath)).GetNSString();
	NSData* CrashData = [NSData dataWithContentsOfFile: Path options: NSMappedRead error: &Error];
	if (CrashData && !Error)
	{
		PLCrashReport* CrashLog = [[PLCrashReport alloc] initWithData: CrashData error: &Error];
		if (CrashLog && !Error)
		{
			NSString* Report = [PLCrashReportTextFormatter stringValueForCrashReport: CrashLog withTextFormat: PLCrashReportTextFormatiOS];
			FString CrashDump = FString(Report);
			[Report writeToFile: Path atomically: YES encoding: NSUTF8StringEncoding error: &Error];
		}
		else
		{
			NSLog(@"****UE5 %@", [Error localizedDescription]);
		}
		[CrashLog release];
	}
	else
	{
		NSLog(@"****UE5 %@", [Error localizedDescription]);
	}
	int ReportFile = open(OutputPath, O_CREAT|O_WRONLY, 0766);
	int DumpFile = open(InputPath, O_RDONLY, 0766);
	if (ReportFile != -1 && DumpFile != -1)
	{
		char Data[PATH_MAX];
		
		int Bytes = 0;
		while((Bytes = read(DumpFile, Data, PATH_MAX)) > 0)
		{
			write(ReportFile, Data, Bytes);
		}
		
		close(DumpFile);
		close(ReportFile);
		
		unlink(InputPath);
	}
	
	//Make sure we close file handles in case we couldn't open both above
	if (DumpFile != -1)
	{
		close(DumpFile);
	}
	if (ReportFile != -1)
	{
		close(ReportFile);
	}
#endif
}

void FApplePlatformCrashContext::WriteLine(int ReportFile, const ANSICHAR* Line)
{
	if( Line != nullptr )
	{
		int64 StringBytes = FCStringAnsi::Strlen(Line);
		write(ReportFile, Line, StringBytes);
	}
	
	// use Windows line terminator
	static ANSICHAR WindowsTerminator[] = "\r\n";
	write(ReportFile, WindowsTerminator, 2);
}

void FApplePlatformCrashContext::WriteUTF16String(int ReportFile, const TCHAR* UTFString4BytesChar, uint32 NumChars)
{
	check(UTFString4BytesChar != NULL || NumChars == 0);
	static_assert(sizeof(TCHAR) == 4 || sizeof(TCHAR) == 2, "Unexpected platform TCHAR size. Revisit this function.");
	
    if(sizeof(TCHAR) == 4)
    {
        for (uint32 Idx = 0; Idx < NumChars; ++Idx)
        {
            write(ReportFile, &UTFString4BytesChar[Idx], 2);
        }
    }
    else // sizeof(TCHAR) == 2
    {
        write(ReportFile, UTFString4BytesChar, NumChars * 2);
    }
}

void FApplePlatformCrashContext::WriteUTF16String(int ReportFile, const TCHAR* UTFString4BytesChar)
{
	uint32 NumChars = FCString::Strlen(UTFString4BytesChar);
	WriteUTF16String(ReportFile, UTFString4BytesChar, NumChars);
}

void FApplePlatformCrashContext::WriteLine(int ReportFile, const TCHAR* Line)
{
	if( Line != nullptr )
	{
		int64 NumChars = FCString::Strlen(Line);
		WriteUTF16String(ReportFile, Line, NumChars);
	}
	
	// use Windows line terminator
	static TCHAR WindowsTerminator[] = TEXT("\r\n");
	WriteUTF16String(ReportFile, WindowsTerminator, 2);
}

ANSICHAR* FApplePlatformCrashContext::ItoANSI(uint64 Val, uint64 Base, uint32 Len)
{
	static ANSICHAR InternalBuffer[64] = {0};
	
	uint64 i = 62;
	int32 pad = Len;
	
	if(Val)
	{
		for(; Val && i ; --i, Val /= Base, --pad)
		{
			InternalBuffer[i] = "0123456789abcdef"[Val % Base];
		}
	}
	else
	{
		InternalBuffer[i--] = '0';
		--pad;
	}
	
	while(pad > 0)
	{
		InternalBuffer[i--] = '0';
		--pad;
	}
	
	return &InternalBuffer[i+1];
}

TCHAR* FApplePlatformCrashContext::ItoTCHAR(uint64 Val, uint64 Base, uint32 Len)
{
	static TCHAR InternalBuffer[64] = {0};
	
	uint64 i = 62;
	int32 pad = Len;
	
	if(Val)
	{
		for(; Val && i ; --i, Val /= Base, --pad)
		{
			InternalBuffer[i] = TEXT("0123456789abcdef")[Val % Base];
		}
	}
	else
	{
		InternalBuffer[i--] = TEXT('0');
		--pad;
	}
	
	while(pad > 0)
	{
		InternalBuffer[i--] = '0';
		--pad;
	}
	
	return &InternalBuffer[i+1];
}


const TCHAR* FApplePlatformCrashContext::GetCallstackProperty() const
{
	return CallstackProperty;
}
