// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ApplePlatformDebugEvents.cpp: Apple platform implementations of File functions
=============================================================================*/

#include "Apple/ApplePlatformDebugEvents.h"
#include "Apple/AppleStringUtils.h"
#include "Apple/AppleDebugEvent.h"
#include "Apple/ApplePlatformTLS.h"
#include "Containers/UnrealString.h"
#include "HAL/IConsoleManager.h"
#include "Math/Color.h"
#include "Misc/ScopeRWLock.h"
#include <sys/kdebug_signpost.h>
#include <sys/syscall.h>

/*------------------------------------------------------------------------------
 Legacy OS Defines.
 ------------------------------------------------------------------------------*/

#ifndef DBG_MACH_CHUD

#define DBG_MACH_CHUD 0x0A
#define DBG_FUNC_NONE 0
#define DBG_FUNC_START 1
#define DBG_FUNC_END 2
#define DBG_APPS              33

#define KDBG_CODE(Class, SubClass, code) (((Class & 0xff) << 24) | ((SubClass & 0xff) << 16) | ((code & 0x3fff)  << 2))
#define APPSDBG_CODE(SubClass,code) KDBG_CODE(DBG_APPS, SubClass, code)

#endif

/*------------------------------------------------------------------------------
 Defines.
 ------------------------------------------------------------------------------*/

#if APPLE_PROFILING_ENABLED

#if PLATFORM_MAC
	#include <os/log.h>
	#include <os/signpost.h>
	
	#undef PLATFORM_MAC
	#undef PLATFORM_IOS
	#undef PLATFORM_TVOS
	#undef PLATFORM_VISIONOS
	
	#define PLATFORM_MAC 1
	#define PLATFORM_IOS 0
	#define PLATFORM_TVOS 0
	#define PLATFORM_VISIONOS 0
#elif PLATFORM_VISIONOS
	#include <os/log.h>
	#include <os/signpost.h>
	
	#undef PLATFORM_MAC
	#undef PLATFORM_IOS
	#undef PLATFORM_TVOS
	#undef PLATFORM_VISIONOS
	
	#define PLATFORM_MAC 0
	#define PLATFORM_IOS 1
	#define PLATFORM_TVOS 0
	#define PLATFORM_VISIONOS 1
#elif PLATFORM_TVOS	
	#include <os/log.h>
	#include <os/signpost.h>
	
	#undef PLATFORM_MAC
	#undef PLATFORM_IOS
	#undef PLATFORM_TVOS
	#undef PLATFORM_VISIONOS
	
	#define PLATFORM_MAC 0
	#define PLATFORM_IOS 1
	#define PLATFORM_TVOS 1
	#define PLATFORM_VISIONOS 0
#elif PLATFORM_IOS
	#include <os/log.h>
	#include <os/signpost.h>
	
	#undef PLATFORM_MAC
	#undef PLATFORM_IOS
	#undef PLATFORM_TVOS
	#undef PLATFORM_VISIONOS
	
	#define PLATFORM_MAC 0
	#define PLATFORM_IOS 1
	#define PLATFORM_TVOS 0
	#define PLATFORM_VISIONOS 0
#else
	#error "Unknown Apple Platform requested for Profiling"
#endif

DEFINE_LOG_CATEGORY(LogInstruments)

/*------------------------------------------------------------------------------
 Console variables.
 ------------------------------------------------------------------------------*/

static int32 GAppleInstrumentsEvents = 0;
static FAutoConsoleVariableRef CVarAppleInstrumentsEvent(
	TEXT("Apple.InstrumentsEvents"),
	GAppleInstrumentsEvents,
	TEXT("Set to true (>0) to emit scoped kdebug events for Instruments, which has a noticeable performance impact or 0 to disable. (Default: 0, off)"),
	ECVF_Default);

/*------------------------------------------------------------------------------
 Implementation.
 ------------------------------------------------------------------------------*/

uint32 FApplePlatformDebugEvents::TLSSlot = FPlatformTLS::AllocTlsSlot();

enum EInstrumentsColors
{
	Blue = 0,
	Green = 1,
	Purple = 2,
	Orange = 3,
	Red = 4,
	Max
};

static uint32 GetInstrumentsColor(FColor const& Color)
{
	uint32 Diff[EInstrumentsColors::Max];
	Diff[EInstrumentsColors::Blue] = FColor::Blue.DWColor() - Color.DWColor();
	Diff[EInstrumentsColors::Green] = FColor::Green.DWColor() - Color.DWColor();
	Diff[EInstrumentsColors::Purple] = FColor::Purple.DWColor() - Color.DWColor();
	Diff[EInstrumentsColors::Orange] = FColor::Orange.DWColor() - Color.DWColor();
	Diff[EInstrumentsColors::Red] = FColor::Red.DWColor() - Color.DWColor();
	
	uint32 Min = 0xffffffff;
	uint32 Index = 0;
	for(uint32 i = 0; i < EInstrumentsColors::Max; i++)
	{
		if (Diff[i] < Min)
		{
			Min = Diff[i];
			Index = i;
		}
	}
	
	return Index;
}

FORCEINLINE_DEBUGGABLE os_log_t GetLog()
{
	static os_log_t Log = os_log_create("com.epicgames.namedevents", "PointsOfInterest");
	return Log;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS

void FApplePlatformDebugEvents::DebugSignPost(uint16 Code, uintptr_t Arg1, uintptr_t Arg2, uintptr_t Arg3, uintptr_t Arg4)
{
	if (GAppleInstrumentsEvents)
	{
		kdebug_signpost(Code, Arg1, Arg2, Arg3, Arg4);
	}
}

void FApplePlatformDebugEvents::DebugSignPostStart(uint16 Code, uintptr_t Arg1, uintptr_t Arg2, uintptr_t Arg3, uintptr_t Arg4)
{
	if (GAppleInstrumentsEvents)
	{
		kdebug_signpost_start(Code, Arg1, Arg2, Arg3, Arg4);
	}
}

void FApplePlatformDebugEvents::DebugSignPostEnd(uint16 Code, uintptr_t Arg1, uintptr_t Arg2, uintptr_t Arg3, uintptr_t Arg4)
{
	if (GAppleInstrumentsEvents)
	{
		kdebug_signpost_end(Code, Arg1, Arg2, Arg3, Arg4);
	}
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS

TArray<FAppleDebugEvent>* FApplePlatformDebugEvents::GetEventStack()
{
	TArray<FAppleDebugEvent>* Current = (TArray<FAppleDebugEvent>*)FPlatformTLS::GetTlsValue(TLSSlot);
	if (!Current)
	{
		Current = new TArray<FAppleDebugEvent>;
		FPlatformTLS::SetTlsValue(TLSSlot, Current);
	}
	check(Current);
	return Current;
}

uint16 FApplePlatformDebugEvents::GetEventCode(FString String)
{
	// Never emit 0 as we use that for the frame marker...
	uint16 Code = 1;
	
	if (String.StartsWith(TEXT("Frame")))
	{
		String = TEXT("Frame");
	}
	else if (String.StartsWith(TEXT("PerObject")))
	{
		String = TEXT("PerObject");
	}
	else if (String.StartsWith(TEXT("PreShadow")))
	{
		String = TEXT("PreShadow");
	}
	
	uint32 Hash = GetTypeHash(String);
	
	static FRWLock RWMutex;
	static TMap<uint32, uint16> Names;
	{
		FRWScopeLock Lock(RWMutex, SLT_ReadOnly);
		uint16* CodePtr = Names.Find(Hash);
		if(CodePtr)
		{
			Code = *CodePtr;
		}
		
		if(!CodePtr)
		{
			Lock.ReleaseReadOnlyLockAndAcquireWriteLock_USE_WITH_CAUTION();
			CodePtr = Names.Find(Hash);
			if(CodePtr)
			{
				Code = *CodePtr;
			}
			else
			{
				Code = (uint16)Names.Num() + 1;
				check(Code < 16384);
				Names.Add(Hash, Code);
				UE_LOGF(LogInstruments, Display, "New Event Code: %u : %ls", (uint32)Code, *String);
			}
		}
	}
	
	return Code;
}

static void AppleBeginNamedEvent(const struct FColor& Color, const ANSICHAR* Text)
{
	if (GAppleInstrumentsEvents)
	{
		NSString* String = [FString(Text).GetNSString() retain];

		FAppleDebugEvent Event;
		Event.Tag = Text;
		Event.Code = os_signpost_id_generate(GetLog());
		Event.Destructor = Block_copy(^{ os_signpost_interval_end(GetLog(), Event.Code, "NamedEvent", "%s", [String UTF8String]); [String release]; });

		FApplePlatformDebugEvents::GetEventStack()->Add(Event);

		os_signpost_interval_begin(GetLog(), Event.Code, "NamedEvent", "%s", [String UTF8String]);
	}
}

void FApplePlatformDebugEvents::BeginNamedEvent(const struct FColor& Color,const TCHAR* Text)
{
	AppleBeginNamedEvent(Color, TCHAR_TO_ANSI(Text));
}

void FApplePlatformDebugEvents::BeginNamedEvent(const struct FColor& Color,const ANSICHAR* Text)
{
	AppleBeginNamedEvent(Color, Text);
}

void FApplePlatformDebugEvents::EndNamedEvent()
{
	if (GAppleInstrumentsEvents)
	{
		TArray<FAppleDebugEvent>* Stack = GetEventStack();
		check(Stack);
		if (Stack->Num() == 0)
		{
			// It is possible (and likely) for the CVar to be flipped mid-tick,
			// meaning some scoped event objects will not have actually pushed
			// anything to the event stack.
			return;
		}
		FAppleDebugEvent Last = Stack->Last();
		Stack->RemoveAt(Stack->Num() - 1);
		Last.Destructor();
		Block_release(Last.Destructor);
	}
}

#endif // APPLE_PROFILING_ENABLED
