// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ApplePlatformFile.mm: Apple platform implementations of File functions
=============================================================================*/

#include "Apple/ApplePlatformFile.h"

#if PLATFORM_MAC
#include "Apple/ScopeAutoreleasePool.h"
#include "Mac/MacSystemIncludes.h"
#elif PLATFORM_IOS
#include "Apple/ScopeAutoreleasePool.h"
#include "IOS/IOSSystemIncludes.h"
#endif

#include "Containers/UnrealString.h"
#include "Containers/StringConv.h"
#include "CoreGlobals.h"
#include "HAL/CriticalSection.h"
#include "HAL/PlatformFile.h"
#include "HAL/PlatformFileCommon.h"
#include "HAL/PlatformTime.h"
#include "Misc/ScopeLock.h"
#include "Misc/ScopeExit.h"
#include "ProfilingDebugging/PlatformFileTrace.h"
#include "HAL/DiskUtilizationTracker.h"
#include "Templates/Function.h"
#include <sys/stat.h>

#include "Async/MappedFileHandle.h"
#include <sys/mman.h>

DEFINE_LOG_CATEGORY_STATIC(LogApplePlatformFile, Log, All);

// make an FTimeSpan object that represents the "epoch" for time_t (from a stat struct)
const FDateTime MacEpoch(1970, 1, 1);

namespace
{
	FFileStatData MacStatToUEFileData(struct stat& FileInfo)
	{
		const bool bIsDirectory = S_ISDIR(FileInfo.st_mode);

		int64 FileSize = -1;
		if (!bIsDirectory)
		{
			FileSize = FileInfo.st_size;
		}

		return FFileStatData(
			MacEpoch + FTimespan::FromSeconds(static_cast<double>(FileInfo.st_ctime)), 
			MacEpoch + FTimespan::FromSeconds(static_cast<double>(FileInfo.st_atime)), 
			MacEpoch + FTimespan::FromSeconds(static_cast<double>(FileInfo.st_mtime)), 
			FileSize,
			bIsDirectory,
			!(FileInfo.st_mode & S_IWUSR)
		);
	}
}
/**
 * Apple version of the file handle registry
 */
class FAppleFileRegistry : public FFileHandleRegistry
{
public:
	FAppleFileRegistry()
		: FFileHandleRegistry(192)
	{
	}

protected:
	virtual FRegisteredFileHandle* PlatformInitialOpenFile(const TCHAR* Filename) override;
	virtual bool PlatformReopenFile(FRegisteredFileHandle* Handle) override;
	virtual void PlatformCloseFile(FRegisteredFileHandle* Handle) override;
};
static FAppleFileRegistry GFileRegistry;

static int32 AppleOpenCommon(const TCHAR* Filename);
static void AppleCloseCommon(int32 FileHandle);

class CORE_API FFileHandleApple : public FRegisteredFileHandle
{
	enum {READWRITE_SIZE = 1024 * 1024};

public:
	FFileHandleApple(int32 InFileHandle, const TCHAR* InFilename, bool bIsReadOnly)
		: FileHandle(InFileHandle)
		, Filename(InFilename)
		, FileOffset(0)
		, FileSize(0)
		, bReadOnly(bIsReadOnly)
	{
		check(FileHandle > -1);

		// Only files opened for read will be managed
		if (bIsReadOnly)
		{
			struct stat FileInfo;
			fstat(FileHandle, &FileInfo);
			FileSize = FileInfo.st_size;
		}
	}
	
	virtual ~FFileHandleApple()
	{
		if (bReadOnly)
		{
			GFileRegistry.UnTrackAndCloseFile(this);
		}
		else
		{
			int Result = fsync(FileHandle);
			if (Result < 0)
			{
				UE_LOGF(LogApplePlatformFile, Error, "Failed to properly flush writable file with errno: %d: %ls",
					errno, UTF8_TO_TCHAR(strerror(errno)));
			}
			
			AppleCloseCommon(FileHandle);
		}
		
		FileHandle = -1;
	}
	
	virtual int64 Tell() override
	{
		if (bReadOnly)
		{
			return FileOffset;
		}
		else
		{
			check(IsValid());
			return lseek(FileHandle, 0, SEEK_CUR);
		}
	}
	
	virtual bool Seek(int64 NewPosition) override
	{
		check(NewPosition >= 0);

		if (bReadOnly)
		{
			FileOffset = NewPosition >= FileSize ? FileSize - 1 : NewPosition;
			return true;
		}
		else
		{
			check(IsValid());
			return lseek(FileHandle, NewPosition, SEEK_SET) != -1;
		}
	}
	
	virtual bool SeekFromEnd(int64 NewPositionRelativeToEnd = 0) override
	{
		check(NewPositionRelativeToEnd <= 0);
		
		if (bReadOnly)
		{
			FileOffset = (NewPositionRelativeToEnd >= FileSize) ? 0 : ( FileSize + NewPositionRelativeToEnd - 1 );
			return true;
		}
		else
		{
			check(IsValid());
			return lseek(FileHandle, NewPositionRelativeToEnd, SEEK_END) != -1;
		}
	}
	
	virtual bool Read(uint8* Destination, int64 BytesToRead) override
	{
		if (bReadOnly)
		{
			// Handle virtual file handles (only in read mode, write mode doesn't use the file handle registry)
			FFileHandleRegistryReadTracker TrackRead(GFileRegistry, *this, true);
			if (!TrackRead.IsValid() || FileOffset >= FileSize)
			{
				return false;
			}

			// seek to the offset on seek? this matches console behavior more closely
			if (lseek(FileHandle, FileOffset, SEEK_SET) == -1)
			{
				return false;
			}
			int64 BytesRead = ReadInternal(Destination, BytesToRead);
			FileOffset += BytesRead;
			return BytesRead == BytesToRead || FileOffset == FileSize;
		}
		else
		{
			int64 Pos = Tell();
			int64 BytesRead = ReadInternal(Destination, BytesToRead);
			if (BytesRead != BytesToRead)
			{
				// If we requested more than was in the file return true if we read all the remaining content
				return BytesRead > 0 && (Pos + BytesRead) == Size();
			}
			return true;
		}
	}
	
	virtual bool ReadAt(uint8* Destination, int64 BytesToRead, int64 Offset) override
	{
		if (BytesToRead < 0 || Offset < 0)
		{
			return false;
		}

		if (BytesToRead == 0)
		{
			return true;
		}

		FFileHandleRegistryReadTracker TrackRead(GFileRegistry, *this, true);
		if (bReadOnly && !TrackRead.IsValid())
		{
			return false;
		}
		
		int64 TotalBytesRead = 0;
		FScopedDiskUtilizationTracker Tracker(BytesToRead, Offset);
		TRACE_PLATFORMFILE_BEGIN_READ(this, FileHandle, Offset, BytesToRead);
		ON_SCOPE_EXIT
		{
			TRACE_PLATFORMFILE_END_READ(this, TotalBytesRead);
		};

		do
		{
			size_t BytesToRead32 = static_cast<size_t>(FMath::Min<int64>(READWRITE_SIZE, BytesToRead));
			ssize_t BytesRead = pread(FileHandle, Destination, BytesToRead32, Offset);

			if (BytesRead == -1 && errno == EINTR)
			{
				continue;
			}
			if (BytesRead <= 0)
			{
				return false;
			}

			TotalBytesRead += BytesRead;
			Offset += BytesRead;
			BytesToRead -= BytesRead;
			Destination += BytesRead;

		} while (BytesToRead > 0);

		return true;
	}
	
	virtual bool Write(const uint8* Source, int64 BytesToWrite) override
	{
		check(IsValid());
		TRACE_PLATFORMFILE_BEGIN_WRITE(this, FileHandle, 0, BytesToWrite);
		int64 TotalBytesWritten = 0;
		while (BytesToWrite > 0)
		{
			int64 ThisSize = FMath::Min<int64>(READWRITE_SIZE, BytesToWrite);
			check(Source);
			int64 BytesWritten = write(FileHandle, Source, ThisSize);
            if (BytesWritten <= 0)
            {
                if (errno == EINTR)
                {
                    continue;
                }
                else
                {
                    TRACE_PLATFORMFILE_END_WRITE(this, TotalBytesWritten);
                    return false;
                }
            }
            TotalBytesWritten += BytesWritten;
			Source += BytesWritten;
			BytesToWrite -= BytesWritten;
		}
		TRACE_PLATFORMFILE_END_WRITE(this, TotalBytesWritten);
		return true;
	}
	virtual bool Flush(const bool bFullFlush = false) override
	{
		check(IsValid());
		
		if (bReadOnly)
		{
			return false;
		}
		
		if (bFullFlush)
		{
			// OS X needs fcntl with F_FULLFSYNC to guarantee a full flush,
			// but still fallback to fsync if fcntl fails
			if (fcntl(FileHandle, F_FULLFSYNC) == 0)
			{
				return true;
			}	
		}
		// HFS+ apparently doesn't always write the updated file size when using fdatasync, so use fsync to be safe
		return fsync(FileHandle) == 0;
	}
	
	virtual bool Truncate(int64 NewSize) override
	{
		check(IsValid());
		
		if (bReadOnly)
		{
			return false;
		}
		int Result = 0;
		do { Result = ftruncate(FileHandle, NewSize); } while (Result < 0 && errno == EINTR);
		return Result == 0;
	}
	virtual int64 Size() override
	{
		if (bReadOnly)
		{
			return FileSize;
		}
		else
		{
			struct stat FileInfo;
			fstat(FileHandle, &FileInfo);
			return FileInfo.st_size;
		}
	}

private:

	int64 ReadInternal(uint8* Destination, int64 BytesToRead)
	{
		check(IsValid());
		int64 MaxReadSize = READWRITE_SIZE;

		int64 TotalBytesRead = 0;
		FScopedDiskUtilizationTracker Tracker(BytesToRead, FileOffset);
		TRACE_PLATFORMFILE_BEGIN_READ(this, FileHandle, FileOffset, BytesToRead);
		ON_SCOPE_EXIT
		{
			TRACE_PLATFORMFILE_END_READ(this, TotalBytesRead);
		};

		while (BytesToRead)
		{
			check(BytesToRead >= 0);
			int64 ThisSize = FMath::Min<int64>(MaxReadSize, BytesToRead);
			check(Destination);
			int64 ThisRead = read(FileHandle, Destination, ThisSize);
			if (ThisRead == -1)
			{
				// Reading from smb can sometimes result in a EINVAL error. Try again a few times with a smaller read buffer.
				if (errno == EINVAL && MaxReadSize > 1024)
				{
					MaxReadSize /= 2;
					continue;
				}
				return TotalBytesRead;
			}
			TotalBytesRead += ThisRead;
			if (ThisRead != ThisSize)
			{
				return TotalBytesRead;
			}
			Destination += ThisSize;
			BytesToRead -= ThisSize;
		}
		return TotalBytesRead;
	}

	// Holds the internal file handle.
	int32 FileHandle;
	
	// Holds the name of the file that this handle represents. Kept around for possible reopen of file.
	FString Filename;

    // Current file offset; valid if a managed handle.
    int64 FileOffset;

    // Cached file size; valid if a managed handle.
    int64 FileSize;
	
	// Whether the file is read-only or permits writes.
	bool bReadOnly;
		
	FORCEINLINE bool IsValid()
	{
		return FileHandle != -1;
	}
	
	friend class FAppleFileRegistry;
};

static SIZE_T FileMappingAlignment = FPlatformMemory::GetConstants().PageSize;

class FMappedFileRegion final : public IMappedFileRegion
{
public:
	class FMappedFileHandle* Parent;
	const uint8* AlignedPtr;
	uint64 AlignedSize;
	
	FMappedFileRegion(const uint8* InMappedPtr, const uint8* InAlignedPtr, size_t InMappedSize, uint64 InAlignedSize, const FString& InDebugFilename, size_t InDebugOffsetIntoFile, class FMappedFileHandle* InParent)
		: IMappedFileRegion(InMappedPtr, InMappedSize, InDebugFilename, InDebugOffsetIntoFile)
		, Parent(InParent)
		, AlignedPtr(InAlignedPtr)
		, AlignedSize(InAlignedSize)
	{
	}
	
	~FMappedFileRegion();
	
	virtual void PreloadHint(int64 PreloadOffset = 0, int64 BytesToPreload = MAX_int64) override
	{
		int64 Size = GetMappedSize();
		const uint8* Ptr = GetMappedPtr();
		int32 FoolTheOptimizer = 0;
		while (Size > 0)
		{
			FoolTheOptimizer += Ptr[0];
			Size -= 4096;
			Ptr += 4096;
		}
		if (FoolTheOptimizer == 0xbadf00d)
		{
			FPlatformProcess::Sleep(0.0f); // this will more or less never happen, but we can't let the optimizer strip these reads
		}
	}
	
};

class FMappedFileHandle final : public IMappedFileHandle
{
	const uint8* MappedPtr;
	FString Filename;
	std::atomic<int32> NumOutstandingRegions;
	int FileHandle;
	
public:
	
	FMappedFileHandle(int InFileHandle, int64 FileSize, const FString& InFilename)
		: IMappedFileHandle(FileSize)
		, MappedPtr(nullptr)
#if !UE_BUILD_SHIPPING
		, Filename(InFilename)
#endif
		, NumOutstandingRegions(0)
		, FileHandle(InFileHandle)
	{
	}

	~FMappedFileHandle()
	{
		// can't delete the file before you delete all outstanding regions
		UE_CLOG(
				NumOutstandingRegions.load(std::memory_order_relaxed),
				LogHAL,
#if UE_BUILD_SHIPPING
				Error,
#else
				Fatal,
#endif
				TEXT("Cleaning mapped file with alive mapped regions: %s"),
				*Filename);	
		close(FileHandle);
	}
	
	virtual IMappedFileRegion* MapRegion(int64 Offset = 0, int64 BytesToMap = MAX_int64, FFileMappingFlags Flags = EMappedFileFlags::ENone) override
	{
		LLM_PLATFORM_SCOPE(ELLMTag::PlatformMMIO);
		check(Offset < GetFileSize()); // don't map zero bytes and don't map off the end of the file
		BytesToMap = FMath::Min<int64>(BytesToMap, GetFileSize() - Offset);
		check(BytesToMap > 0); // don't map zero bytes
		
		const int64 AlignedOffset = AlignDown(Offset, FileMappingAlignment);
		//File mapping can extend beyond file size. It's OK, kernel will just fill any leftover page data with zeros
		const int64 AlignedSize = Align(BytesToMap + Offset - AlignedOffset, FileMappingAlignment);
		
		int Protection = PROT_READ;
		int InternalFlags = 0;
		if (EnumHasAnyFlags(Flags.Flags, EMappedFileFlags::EFileWritable))
		{
			Protection |= PROT_WRITE;
			InternalFlags |= MAP_SHARED;
		}
		else
		{
			InternalFlags |= MAP_PRIVATE;
		}

		const uint8* AlignedMapPtr = (const uint8 *)mmap(NULL, AlignedSize, Protection, InternalFlags, FileHandle, AlignedOffset);
		if (AlignedMapPtr == (const uint8*)-1 || AlignedMapPtr == nullptr)
		{
			UE_LOGF(LogHAL, Warning, "Failed to map memory %ls, error is %d, aligned size = %lld, aligned offset = %lld, internal flags = %d", *Filename, errno, AlignedSize, AlignedOffset, InternalFlags);
			return nullptr;
		}
		LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelAlloc(ELLMTracker::Platform, AlignedMapPtr, AlignedSize));

		// create a mapping for this range
		const uint8* MapPtr = AlignedMapPtr + Offset - AlignedOffset;
		FMappedFileRegion* Result = new FMappedFileRegion(MapPtr, AlignedMapPtr, BytesToMap, AlignedSize, Filename, Offset, this);

		NumOutstandingRegions.fetch_add(1, std::memory_order_relaxed);

		return Result;
	}
	
	void UnMap(FMappedFileRegion* Region)
	{
		LLM_PLATFORM_SCOPE(ELLMTag::PlatformMMIO);
		
		int32 OldNumOutstandingRegions = NumOutstandingRegions.fetch_sub(1, std::memory_order_relaxed);
		check(OldNumOutstandingRegions > 0);	
		
		LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelFree(ELLMTracker::Platform, (void*)Region->AlignedPtr));
		int Res = munmap((void*)Region->AlignedPtr, Region->AlignedSize);
		checkf(Res == 0, TEXT("Failed to unmap, error is %d, errno is %d [params: %llx, %lld]"), Res, errno, MappedPtr, GetFileSize());
	}
};

FMappedFileRegion::~FMappedFileRegion()
{
	Parent->UnMap(this);
}

/**
 * Mac File I/O implementation
**/
FString FApplePlatformFile::NormalizeFilename(const TCHAR* Filename)
{
	FString Result(Filename);
	Result.ReplaceInline(TEXT("\\"), TEXT("/"), ESearchCase::CaseSensitive);
	return Result;
}
FString FApplePlatformFile::NormalizeDirectory(const TCHAR* Directory)
{
	FString Result(Directory);
	Result.ReplaceInline(TEXT("\\"), TEXT("/"), ESearchCase::CaseSensitive);
	return Result;
}

bool FApplePlatformFile::FileExists(const TCHAR* Filename)
{
	struct stat FileInfo;
	if (Stat(Filename, &FileInfo) != -1)
	{
		return S_ISREG(FileInfo.st_mode);
	}
	return false;
}
int64 FApplePlatformFile::FileSize(const TCHAR* Filename)
{
	struct stat FileInfo;
	FileInfo.st_size = -1;
	if (Stat(Filename, &FileInfo) != 0)
	{
		return -1;
	}
	
	// make sure to return -1 for directories
	if (S_ISDIR(FileInfo.st_mode))
	{
		FileInfo.st_size = -1;
	}
	return FileInfo.st_size;
}
bool FApplePlatformFile::DeleteFile(const TCHAR* Filename)
{
	return unlink(TCHAR_TO_UTF8(*NormalizeFilename(Filename))) == 0;
}
bool FApplePlatformFile::IsReadOnly(const TCHAR* Filename)
{
	if (access(TCHAR_TO_UTF8(*NormalizeFilename(Filename)), F_OK) == -1)
	{
		return false; // file doesn't exist
	}
	if (access(TCHAR_TO_UTF8(*NormalizeFilename(Filename)), W_OK) == -1)
	{
		return errno == EACCES;
	}
	return false;
}
bool FApplePlatformFile::MoveFile(const TCHAR* To, const TCHAR* From)
{
	int32 Result = rename(TCHAR_TO_UTF8(*NormalizeFilename(From)), TCHAR_TO_UTF8(*NormalizeFilename(To)));
	if (Result == -1 && errno == EXDEV)
	{
		// Copy the file if rename failed because To and From are on different file systems
		if (CopyFile(To, From))
		{
			DeleteFile(From);
			Result = 0;
		}
	}
	return Result != -1;
}
bool FApplePlatformFile::SetReadOnly(const TCHAR* Filename, bool bNewReadOnlyValue)
{
	struct stat FileInfo;
	if (Stat(Filename, &FileInfo) == 0)
	{
		if (bNewReadOnlyValue)
		{
			FileInfo.st_mode &= ~S_IWUSR;
		}
		else
		{
			FileInfo.st_mode |= S_IWUSR;
		}
		return chmod(TCHAR_TO_UTF8(*NormalizeFilename(Filename)), FileInfo.st_mode) == 0;
	}
	return false;
}


FDateTime FApplePlatformFile::GetTimeStamp(const TCHAR* Filename)
{
	// get file times
	struct stat FileInfo;
	if(Stat(Filename, &FileInfo) != 0)
	{
		return FDateTime::MinValue();
	}

	// convert _stat time to FDateTime
	FTimespan TimeSinceEpoch(0, 0, FileInfo.st_mtime);
	return MacEpoch + TimeSinceEpoch;

}

void FApplePlatformFile::SetTimeStamp(const TCHAR* Filename, const FDateTime DateTime)
{
	// get file times
	struct stat FileInfo;
	if (Stat(Filename, &FileInfo) != 0)
	{
		return;
	}

	// change the modification time only
	struct utimbuf Times;
	Times.actime = FileInfo.st_atime;
	Times.modtime = (time_t)(DateTime - MacEpoch).GetTotalSeconds();
	utime(TCHAR_TO_UTF8(*NormalizeFilename(Filename)), &Times);
}

FDateTime FApplePlatformFile::GetAccessTimeStamp(const TCHAR* Filename)
{
	// get file times
	struct stat FileInfo;
	if(Stat(Filename, &FileInfo) == -1)
	{
		return FDateTime::MinValue();
	}

	// convert _stat time to FDateTime
	FTimespan TimeSinceEpoch(0, 0, FileInfo.st_atime);
	return MacEpoch + TimeSinceEpoch;
}

FString FApplePlatformFile::GetFilenameOnDisk(const TCHAR* Filename)
{
	return Filename;
}

ESymlinkResult FApplePlatformFile::IsSymlink(const TCHAR* Filename)
{
	struct stat FileInfo;
	if (Stat(Filename, &FileInfo) != -1 && S_ISLNK(FileInfo.st_mode))
	{
		return ESymlinkResult::Symlink;
	}
	return ESymlinkResult::NonSymlink;
}

IFileHandle* FApplePlatformFile::OpenRead(const TCHAR* Filename, bool bAllowWrite)
{
	return GFileRegistry.InitialOpenFile(*NormalizeFilename(Filename));
}

IFileHandle* FApplePlatformFile::OpenWrite(const TCHAR* Filename, bool bAppend, bool bAllowRead)
{
	int Flags = O_CREAT | O_CLOEXEC;
	
	if (bAllowRead)
	{
		Flags |= O_RDWR;
	}
	else
	{
		Flags |= O_WRONLY;
	}
	
	const FString NormalizedFilename = NormalizeFilename(Filename);

	TRACE_PLATFORMFILE_BEGIN_OPEN(*NormalizedFilename);
	int32 Handle = open(TCHAR_TO_UTF8(*NormalizedFilename), Flags, GetModeFlagsForOpenWrite());
	
	if (Handle != -1)
	{
		TRACE_PLATFORMFILE_END_OPEN(Handle);
#if PLATFORM_MAC && UE_EDITOR && !UE_BUILD_SHIPPING
		// No blocking attempt EXclusive lock, failure means we should not have opened the file for writing, protect against multiple instances and client/server versions
		if(flock(Handle, LOCK_NB | LOCK_EX) != 0)
		{
			TRACE_PLATFORMFILE_BEGIN_CLOSE(Handle);
			int CloseResult = close(Handle);
#if PLATFORMFILETRACE_ENABLED
			if (CloseResult >= 0)
			{
				TRACE_PLATFORMFILE_END_CLOSE(Handle);
			}
			else
			{
				TRACE_PLATFORMFILE_FAIL_CLOSE(Handle);
			}
#else
			(void)CloseResult;
#endif
			return nullptr;
		}
		
		// We have created the writer, if reading is required downgrade the lock to SHared
		if(bAllowRead)
		{
			flock(Handle, LOCK_NB | LOCK_SH);
		}
#endif
		
		// Truncate after locking as lock may fail - don't use O_TRUNC in open flags
		if(!bAppend)
		{
			ftruncate(Handle, 0);
		}

		FFileHandleApple* FileHandleApple = new FFileHandleApple(Handle, *NormalizeDirectory(Filename), false);

		if (bAppend)
		{
			FileHandleApple->SeekFromEnd(0);
		}
		return FileHandleApple;
	}
	else
	{
		TRACE_PLATFORMFILE_FAIL_OPEN(*NormalizedFilename);
		return nullptr;
	}
}

FOpenMappedResult FApplePlatformFile::OpenMappedEx(const TCHAR* Filename, EOpenReadFlags OpenOptions, int64 MaximumSize)
{
	EOpenMappedFlags ActualOpenOptions = EOpenMappedFlags::None;
	if (EnumHasAnyFlags(OpenOptions, EOpenReadFlags::AllowWrite))
	{
		ActualOpenOptions |= EOpenMappedFlags::OpenForWrite;
		ActualOpenOptions |= EOpenMappedFlags::OpenExisting;
	}

	return OpenMappedEx2(Filename, ActualOpenOptions, MaximumSize);
}

FOpenMappedResult FApplePlatformFile::OpenMappedEx2(const TCHAR* Filename, EOpenMappedFlags OpenOptions, int64 MaximumSize)
{
	const FString NormalizedFilename = NormalizeFilename(Filename);

	FString FinalPath;
	const int32 Handle = OpenFileForMemMap(NormalizedFilename, OpenOptions, FinalPath);
	if (Handle == -1)
	{
		return MakeError(FString::Printf(TEXT("FApplePlatformFile::OpenMappedEx failed to open file '%s' in '%s' mode"), *NormalizedFilename, EnumHasAnyFlags(OpenOptions, EOpenMappedFlags::OpenForWrite) ? TEXT("write") : TEXT("read")));
	}

	struct stat FileInfo;
	FileInfo.st_size = -1;
	const int StatResult = fstat(Handle, &FileInfo);
	if (StatResult == -1)
	{
		close(Handle);
		return MakeError(FString::Printf(TEXT("FApplePlatformFile::OpenMappedEx failed to get file info for file '%s'"), *NormalizedFilename));
	}
	
	if (EnumHasAnyFlags(OpenOptions, EOpenMappedFlags::OpenForWrite))
	{
		if (MaximumSize < 1)
		{
			if (EnumHasAnyFlags(OpenOptions, EOpenMappedFlags::OpenExisting))
			{
				if (FileInfo.st_size < 1)
				{
					close(Handle);
					return MakeError(TEXT("File size is 0"));
				}

				MaximumSize = FileInfo.st_size;
			}
			else
			{
				close(Handle);
				return MakeError(TEXT("No maximum size provided."));
			}
		}
		else if (MaximumSize > FileInfo.st_size)
		{
			int TruncResult = 0;
			do { TruncResult = ftruncate(Handle, MaximumSize); } while (TruncResult < 0 && errno == EINTR);

			if (TruncResult == -1)
			{
				close(Handle);
				return MakeError(FString::Printf(TEXT("FApplePlatformFile::OpenMappedEx ftruncate failed to extend file '%s'"), *NormalizedFilename));
			}
		}
	}
	else
	{
		if (FileInfo.st_size < 1)
		{
			close(Handle);
			return MakeError(TEXT("File size is 0"));
		}

		if (MaximumSize < 1 || MaximumSize > FileInfo.st_size)
		{
			MaximumSize = FileInfo.st_size;
		}
	}

	return MakeValue(new FMappedFileHandle(Handle, MaximumSize, FinalPath));
}

int32 FApplePlatformFile::OpenFileForMemMap(const FString& NormalizedFilename, EOpenMappedFlags OpenOptions, FString& OutFinalPath)
{
	OutFinalPath = NormalizedFilename;

	int Flags = O_RDONLY | O_CLOEXEC;
	if (EnumHasAnyFlags(OpenOptions, EOpenMappedFlags::OpenForWrite))
	{
		Flags = O_RDWR | O_CLOEXEC;
		if (!EnumHasAnyFlags(OpenOptions, EOpenMappedFlags::OpenExisting))
		{
			Flags |= O_CREAT;
			Flags |= O_TRUNC;
		}
	}

	int32 Handle = -1;

	if (EnumHasAnyFlags(OpenOptions, EOpenMappedFlags::OpenForWrite))
	{
		Handle = open(TCHAR_TO_UTF8(*NormalizedFilename), Flags, GetModeFlagsForOpenWrite());
	}
	else
	{
		Handle = open(TCHAR_TO_UTF8(*NormalizedFilename), Flags);
	}

	return Handle;
}

bool FApplePlatformFile::DirectoryExists(const TCHAR* Directory)
{
	struct stat FileInfo;
	if (Stat(Directory, &FileInfo) != -1)
	{
		return S_ISDIR(FileInfo.st_mode);
	}
	return false;
}

bool FApplePlatformFile::CreateDirectory(const TCHAR* Directory)
{
	SCOPED_AUTORELEASE_POOL;

	// Using native functions on program targets to be able to use detour and remote helpers
#if IS_PROGRAM
	if (mkdir(TCHAR_TO_UTF8(*NormalizeFilename(Directory)), 0775) == 0)
		return true;
	return errno == EEXIST;
#else
	CFStringRef CFDirectory = FPlatformString::TCHARToCFString(*NormalizeFilename(Directory));
	bool Result = [[NSFileManager defaultManager] createDirectoryAtPath:(NSString*)CFDirectory withIntermediateDirectories:true attributes:nil error:nil];
	CFRelease(CFDirectory);
	return Result;
#endif
}

bool FApplePlatformFile::DeleteDirectory(const TCHAR* Directory)
{
	return rmdir(TCHAR_TO_UTF8(*NormalizeFilename(Directory))) == 0;
}

FFileStatData FApplePlatformFile::GetStatData(const TCHAR* FilenameOrDirectory)
{
	struct stat FileInfo;
	if (Stat(FilenameOrDirectory, &FileInfo) != -1)
	{
		return MacStatToUEFileData(FileInfo);
	}

	return FFileStatData();
}

bool FApplePlatformFile::IterateDirectory(const TCHAR* Directory, FDirectoryVisitor& Visitor)
{
	const FString DirectoryStr = Directory;
	const FString NormalizedDirectoryStr = NormalizeFilename(Directory);

	return IterateDirectoryCommon(Directory, [&Visitor, &DirectoryStr, &NormalizedDirectoryStr](struct dirent* InEntry) -> bool
	{
		SCOPED_AUTORELEASE_POOL;

		// Normalize any unicode forms so we match correctly
		const FString NormalizedFilename = UTF8_TO_TCHAR(([[[NSString stringWithUTF8String:InEntry->d_name] precomposedStringWithCanonicalMapping] cStringUsingEncoding:NSUTF8StringEncoding]));

		// Figure out whether it's a directory. Some protocols (like NFS) do not voluntarily return this as part of the directory entry, and need to be queried manually.
		bool bIsDirectory = (InEntry->d_type == DT_DIR);
		if (InEntry->d_type == DT_UNKNOWN || InEntry->d_type == DT_LNK)
		{
			struct stat StatInfo;
			if (stat(TCHAR_TO_UTF8(*(NormalizedDirectoryStr / NormalizedFilename)), &StatInfo) == 0)
			{
				bIsDirectory = S_ISDIR(StatInfo.st_mode);
			}
		}

		return Visitor.CallShouldVisitAndVisit(*(DirectoryStr / NormalizedFilename), bIsDirectory);
	});
}

bool FApplePlatformFile::IterateDirectoryStat(const TCHAR* Directory, FDirectoryStatVisitor& Visitor)
{
	const FString DirectoryStr = Directory;
	const FString NormalizedDirectoryStr = NormalizeFilename(Directory);

	return IterateDirectoryCommon(Directory, [&Visitor, &DirectoryStr, &NormalizedDirectoryStr](struct dirent* InEntry) -> bool
	{
		SCOPED_AUTORELEASE_POOL;

		// Normalize any unicode forms so we match correctly
		const FString NormalizedFilename = UTF8_TO_TCHAR(([[[NSString stringWithUTF8String:InEntry->d_name] precomposedStringWithCanonicalMapping] cStringUsingEncoding:NSUTF8StringEncoding]));

		struct stat StatInfo;
		if (stat(TCHAR_TO_UTF8(*(NormalizedDirectoryStr / NormalizedFilename)), &StatInfo) == 0)
		{
			return Visitor.CallShouldVisitAndVisit(*(DirectoryStr / NormalizedFilename), MacStatToUEFileData(StatInfo));
		}

		return true;
	});
}

bool FApplePlatformFile::IterateDirectoryCommon(const TCHAR* Directory, const TFunctionRef<bool(struct dirent*)>& Visitor)
{
	bool Result = false;
	DIR* Handle = opendir(Directory[0] ? TCHAR_TO_UTF8(Directory) : ".");
	if (Handle)
	{
		Result = true;
		struct dirent *Entry;
		while (Result && (Entry = readdir(Handle)) != NULL)
		{
			if (FCStringAnsi::Strcmp(Entry->d_name, ".") && FCStringAnsi::Strcmp(Entry->d_name, "..") && FCStringAnsi::Strcmp(Entry->d_name, ".DS_Store"))
			{
                Result = Visitor(Entry);
			}
		}
		closedir(Handle);
	}
	return Result;
}

bool FApplePlatformFile::CopyFile(const TCHAR* To, const TCHAR* From, EPlatformFileRead ReadFlags, EPlatformFileWrite WriteFlags)
{
	bool Result = IPlatformFile::CopyFile(To, From, ReadFlags, WriteFlags);
	if (Result)
	{
		struct stat FileInfo;
		if (Stat(From, &FileInfo) == 0)
		{
			FileInfo.st_mode |= S_IWUSR;
			chmod(TCHAR_TO_UTF8(*NormalizeFilename(To)), FileInfo.st_mode);
		}
	}
	return Result;
}

int32 FApplePlatformFile::Stat(const TCHAR* Filename, struct stat* OutFileInfo)
{
	return stat(TCHAR_TO_UTF8(*NormalizeFilename(Filename)), OutFileInfo);
}

int FApplePlatformFile::GetModeFlagsForOpenWrite() const
{
	return S_IRUSR | S_IWUSR | S_IRGRP | S_IWGRP | S_IROTH | S_IWOTH;
}

int32 AppleOpenCommon(const TCHAR* Filename)
{
	TRACE_PLATFORMFILE_BEGIN_OPEN(Filename);
	int32 Handle = open(TCHAR_TO_UTF8(Filename), O_RDONLY | O_CLOEXEC);
	if (Handle != -1)
	{
		TRACE_PLATFORMFILE_END_OPEN(Handle);
#if PLATFORM_MAC && UE_EDITOR && !UE_BUILD_SHIPPING
		// No blocking attempt shared lock, failure means we should not have opened the file for reading, protect against multiple instances and client/server versions
		if(flock(Handle, LOCK_NB | LOCK_SH) != 0)
		{
			TRACE_PLATFORMFILE_BEGIN_CLOSE(Handle);
			int CloseResult = close(Handle);
#if PLATFORMFILETRACE_ENABLED
			if (CloseResult >= 0)
			{
				TRACE_PLATFORMFILE_END_CLOSE(Handle);
			}
			else
			{
				TRACE_PLATFORMFILE_FAIL_CLOSE(Handle);
			}
#else
			(void)CloseResult;
#endif
			return -1;
		}
#endif // PLATFORM_MAC && UE_EDITOR && !UE_BUILD_SHIPPING
		return Handle;
	}
	else
	{
		TRACE_PLATFORMFILE_FAIL_OPEN(Filename);
		return -1;
	}
}

void AppleCloseCommon(int32 FileHandle)
{
	TRACE_PLATFORMFILE_BEGIN_CLOSE(FileHandle);
	flock(FileHandle, LOCK_UN | LOCK_NB);
	int CloseResult = close(FileHandle);
	if (CloseResult >= 0)
	{
		TRACE_PLATFORMFILE_END_CLOSE(FileHandle);
	}
	else
	{
		TRACE_PLATFORMFILE_FAIL_CLOSE(FileHandle);
		UE_LOGF(LogInit, Warning, "Failed to properly close file with errno: %d: %ls",
			errno, UTF8_TO_TCHAR(strerror(errno)));
	}
}

FRegisteredFileHandle* FAppleFileRegistry::PlatformInitialOpenFile(const TCHAR* Filename)
{
	const int32 Handle = AppleOpenCommon(Filename);
	if (Handle != -1)
	{
		return new FFileHandleApple(Handle, Filename, true);;
	}
	else
	{
		return nullptr;
	}
}

bool FAppleFileRegistry::PlatformReopenFile(FRegisteredFileHandle* Handle)
{
	FFileHandleApple* AppleHandle = static_cast<FFileHandleApple*>(Handle);
	
	bool bSuccess = true;
	AppleHandle->FileHandle = AppleOpenCommon(*(AppleHandle->Filename));
	if (AppleHandle->FileHandle != -1)
	{
		if (lseek(AppleHandle->FileHandle, AppleHandle->FileOffset, SEEK_SET) == -1)
		{
			UE_LOGF(LogApplePlatformFile, Warning, "Could not seek to the previous position on handle for file '%ls'", *(AppleHandle->Filename));
			bSuccess = false;
		}
	}
	else
	{
		UE_LOGF(LogApplePlatformFile, Warning, "Could not reopen handle for file '%ls'", *(AppleHandle->Filename));
		bSuccess = false;
	}

	return bSuccess;
}

void FAppleFileRegistry::PlatformCloseFile(FRegisteredFileHandle* Handle)
{
	FFileHandleApple* AppleHandle = static_cast<FFileHandleApple*>(Handle);
	AppleCloseCommon(AppleHandle->FileHandle);
}
