// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// TODO: Use os_sync_wait_on_address/os_sync_wake_by_address_all in macOS 14.4+, iOS/iPadOS/tvOS 17.4+, visionOS 1.1+.

#include "GenericPlatform/GenericPlatformManualResetEvent.h"
namespace UE::HAL::Private { using FApplePlatformManualResetEvent = FGenericPlatformManualResetEvent; }
