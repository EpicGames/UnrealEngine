// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ApplePlatformMemory.h: Apple platform memory functions common across all Apple OSes
=============================================================================*/

#include "Apple/ApplePlatformMemory.h"

#include "Apple/AppleMemoryObject.h"

#if PLATFORM_MAC
#include "Mac/MacSystemIncludes.h"
#elif PLATFORM_IOS
#include "IOS/IOSSystemIncludes.h"
#endif

#include "HAL/PlatformMemory.h"
#include "HAL/PlatformMath.h"
#include "HAL/UnrealMemory.h"
#include "HAL/LowLevelMemTracker.h"
#include "Templates/AlignmentTemplates.h"

#include <stdlib.h>
#include "Misc/AssertionMacros.h"
#include "Misc/CoreStats.h"
#include "HAL/MallocAnsi.h"
#include "HAL/MallocBinned.h"
#include "HAL/MallocBinned2.h"
#include "HAL/MallocBinned3.h"
#include "HAL/MallocMimalloc.h"
#include "CoreGlobals.h"

#include <stdlib.h>
#include <sys/param.h>
#include <sys/mman.h>
#include <sys/mount.h>
#include <objc/runtime.h>
#if PLATFORM_MAC
#include <libproc.h>
#endif
#if PLATFORM_IOS
#include "IOS/IOSPlatformMisc.h"
#include <os/proc.h>
#endif
#include <CoreFoundation/CFBase.h>
#include <mach/vm_map.h>
#include <mach/vm_region.h>
#include <mach/vm_statistics.h>
#include <mach/vm_page_size.h>
#include "HAL/LowLevelMemTracker.h"
#include "Apple/AppleLLM.h"

#if PLATFORM_IOS
#include <sys/resource.h> // On iOS, iPadOS and tvOS.
extern "C" int proc_pid_rusage(int pid, int flavor, rusage_info_t *buffer)  
	__OSX_AVAILABLE_STARTING(__MAC_10_9, __IPHONE_7_0);
#endif

NS_ASSUME_NONNULL_BEGIN

// Enable to use XCode Instuments hooks to propagate UE allocs.
// It has impact on runtime performance, disabled by default. 
#define USE_APPLE_SUPPORT_INSTRUMENTED_ALLOCS (0 && APPLE_SUPPORT_INSTRUMENTED_ALLOCS)

// This name is used in XCode instruments
enum class MmapTag : int32
{
	BinnedAllocFromOS			= VM_MAKE_TAG(VM_MEMORY_APPLICATION_SPECIFIC_1 + 0), // "Memory Tag 240"
	AllocateVirtualMemoryBlock	= VM_MAKE_TAG(VM_MEMORY_APPLICATION_SPECIFIC_1 + 1), // "Memory Tag 241"
	LLMAlloc					= VM_MAKE_TAG(VM_MEMORY_APPLICATION_SPECIFIC_1 + 2), // "Memory Tag 242"
};

static void* MmapWithTag(void* __nullable Addr, const size_t Len, const int Prot, const int Flags, const MmapTag Tag, const off_t Offset)
{
	// Replace Tag with -1 to disable
	return mmap(Addr, Len, Prot, Flags, (int32)Tag, Offset);
}

static void* FApplePlatformAllocatorAllocate(CFIndex AllocSize, CFOptionFlags Hint, void* Info)
{
	void* Mem = FMemory::Malloc(AllocSize, 16);
	return Mem;
}

static void* FApplePlatformAllocatorReallocate(void* Ptr, CFIndex Newsize, CFOptionFlags Hint, void* Info)
{
	void* Mem = FMemory::Realloc(Ptr, Newsize, 16);
	return Mem;
}

static void FApplePlatformAllocatorDeallocate(void* Ptr, void* Info)
{
	return FMemory::Free(Ptr);
}

static CFIndex FApplePlatformAllocatorPreferredSize(CFIndex Size, CFOptionFlags Hint, void* Info)
{
	return FMemory::QuantizeSize(Size);
}

void FApplePlatformMemory::ConfigureDefaultCFAllocator(void)
{
	// Configure CoreFoundation's default allocator to use our allocation routines too.
	CFAllocatorContext AllocatorContext;
	AllocatorContext.version = 0;
	AllocatorContext.info = nullptr;
	AllocatorContext.retain = nullptr;
	AllocatorContext.release = nullptr;
	AllocatorContext.copyDescription = nullptr;
	AllocatorContext.allocate = &FApplePlatformAllocatorAllocate;
	AllocatorContext.reallocate = &FApplePlatformAllocatorReallocate;
	AllocatorContext.deallocate = &FApplePlatformAllocatorDeallocate;
	AllocatorContext.preferredSize = &FApplePlatformAllocatorPreferredSize;
	
	CFAllocatorRef Alloc = CFAllocatorCreate(kCFAllocatorDefault, &AllocatorContext);
	CFAllocatorSetDefault(Alloc);
}

vm_address_t FApplePlatformMemory::NanoRegionStart = 0;
vm_address_t FApplePlatformMemory::NanoRegionEnd = 0;

void FApplePlatformMemory::NanoMallocInit()
{
	/*
		iOS reserves 512MB of address space for 'nano' allocations (allocations <= 256 bytes)
		Nano malloc has buckets for sizes 16, 32, 48....256
		The number of buckets and their sizes are fixed and do not grow
		We'll walk through the buckets and ask the VM about the backing regions
		We may have to check several sizes because we can hit a case where all the buckets
		for a specific size are full - which means malloc will put that allocation into
		the MALLOC_TINY region instead.
	 
		The OS always tags the nano VM region with user_tag == VM_MEMORY_MALLOC_NANO (which is 11)
	 
		Being apple this is subject to change at any time and may be different in debug modes, etc.
		We'll fall back to the UE allocators if we can't find the nano region.
	 
		We want to detect this as early as possible, before any of the memory system is initialized.
	*/
	
	NanoRegionStart = 0;
	NanoRegionEnd = 0;
	
	size_t MallocSize = 16;
	while(true)
	{
		void* NanoMalloc = ::malloc(MallocSize);
		FMemory::Memzero(NanoMalloc, MallocSize); // This will wire the memory. Shouldn't be necessary but better safe than sorry.
	
		kern_return_t kr = KERN_SUCCESS;
		vm_address_t address = (vm_address_t)(NanoMalloc);
		vm_size_t regionSizeInBytes = 0;
		mach_port_t regionObjectOut;
		vm_region_extended_info_data_t regionInfo;
		mach_msg_type_number_t infoSize = VM_REGION_EXTENDED_INFO_COUNT;
		kr = vm_region_64(mach_task_self(), &address, &regionSizeInBytes, VM_REGION_EXTENDED_INFO, (vm_region_info_t) &regionInfo, &infoSize, &regionObjectOut);
		check(kr == KERN_SUCCESS);
		
		::free(NanoMalloc);
		
		if(regionInfo.user_tag == VM_MEMORY_MALLOC_NANO)
		{
			uint8_t* Start = (uint8_t*) address;
			uint8_t* End = Start + regionSizeInBytes;
			NanoRegionStart = address;
			NanoRegionEnd = (vm_address_t) End;
			break;
		}
		
		MallocSize += 16;
		
		if(MallocSize > 256)
		{
			// Nano region wasn't found.
			// We'll fall back to the UE allocator
			// This can happen when using various tools
			check(NanoRegionStart == 0 && NanoRegionEnd == 0);
			break;
		}
	}
	
//	if(NanoRegionStart == 0 && NanoRegionEnd == 0)
//	{
//		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("WARNING: No nano malloc region found. We will always use UE allocators\n"));
//	}
//	else
//	{
//		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Detected nanozone %p - %p\n"), (void*) NanoRegionStart, (void*) NanoRegionEnd);
//	}
}

void FApplePlatformMemory::Init()
{
	// Only allow this method to be called once
	{
		static bool bInitDone = false;
		if (bInitDone)
			return;
		bInitDone = true;
	}

	FGenericPlatformMemory::Init();
    
	LLM(AppleLLM::Initialise());

#if PLATFORM_MAC
	// Following Windows' lead on disabling program size. The default implementation
	// of program size (in FLLMGlobals::InitializeProgramSize) right now is 
	// FPlatformMemoryStats::TotalPhysical - FPlatformMemoryStats::AvailablePhysical.
	// Unless we are running in an environment primarily designed around one process,
	// this is incorrect as both of these numbers are system-level stats and could include
	// memory taken up by other processes.
	LLM(FLowLevelMemTracker::Get().SetProgramSize(0));
#endif
	
	const FPlatformMemoryConstants& MemoryConstants = FPlatformMemory::GetConstants();
	UE_LOGF(LogInit, Log, "Memory total: Physical=%.1fGB (%dGB approx) Pagefile=%.1fGB Virtual=%.1fGB",
	   float(static_cast<double>(MemoryConstants.TotalPhysical)/1024.0/1024.0/1024.0),
	   MemoryConstants.TotalPhysicalGB,
	   float(static_cast<double>(MemoryConstants.TotalVirtual-MemoryConstants.TotalPhysical)/1024.0/1024.0/1024.0),
	   float(static_cast<double>(MemoryConstants.TotalVirtual)/1024.0/1024.0/1024.0) );
	
}

#ifndef UE_APPLE_USE_MIMALLOC_MIN_FREE_RAM_MB
#	define UE_APPLE_USE_MIMALLOC_MIN_FREE_RAM_MB 0
#endif

void SelectAllocatorToUse(const FPlatformMemoryStats& MemStats)
{
    // force Ansi allocator in particular cases
    if(getenv("UE4_FORCE_MALLOC_ANSI") != nullptr)
    {
		NSLog(@"UE4_FORCE_MALLOC_ANSI is set, using Ansi allocator.\n");
        FPlatformMemory::AllocatorToUse = FPlatformMemory::EMemoryAllocatorToUse::Ansi;
        return;
    }
    if (FORCE_ANSI_ALLOCATOR)
    {
		NSLog(@"FORCE_ANSI_ALLOCATOR defined, using Ansi allocator.\n");
		FPlatformMemory::AllocatorToUse = FPlatformMemory::EMemoryAllocatorToUse::Ansi;
        return;
    }
	if (!ENABLE_PGO_PROFILE && UE_APPLE_USE_MIMALLOC_MIN_FREE_RAM_MB && ((MemStats.AvailablePhysical / 1024 / 1024) >= UE_APPLE_USE_MIMALLOC_MIN_FREE_RAM_MB))
	{
		NSLog(@"Using mimalloc allocator.\n");
		FPlatformMemory::AllocatorToUse = FPlatformMemory::EMemoryAllocatorToUse::Mimalloc;
		return;
	}
	if (USE_MALLOC_BINNED3)
	{
		if (!FApplePlatformMemory::CanOverallocateVirtualMemory())
		{
			NSLog(@"MallocBinned3 requested but com.apple.developer.kernel.extended-virtual-addressing entitlement not found. Check your entitlements. Falling back to Ansi.\n");
			FPlatformMemory::AllocatorToUse = FPlatformMemory::EMemoryAllocatorToUse::Ansi;
			return;
		}

		NSLog(@"Using MallocBinned3 allocator.\n");
		FPlatformMemory::AllocatorToUse = FPlatformMemory::EMemoryAllocatorToUse::Binned3;
		return;
	}
    else if (USE_MALLOC_BINNED2)
    {
		NSLog(@"Using MallocBinned2 allocator.\n");
		FPlatformMemory::AllocatorToUse = FPlatformMemory::EMemoryAllocatorToUse::Binned2;
        return;
    }
    else
    {
		NSLog(@"Defaulting to Ansi allocator.\n");
		FPlatformMemory::AllocatorToUse = FPlatformMemory::EMemoryAllocatorToUse::Ansi;
        return;
    }
}

#if !defined(UE_APPLE_DISABLE_MALLOC_ROUTER)
#	define UE_APPLE_DISABLE_MALLOC_ROUTER 0
#endif

// We need to create this decorator to route any calls to delete if that pointer was allocated from the crt allocator as some system libs mix allocator calls
template <class T>
struct FAppleMallocRouter final : public T
{
#if !UE_APPLE_DISABLE_MALLOC_ROUTER
	virtual void* Realloc(void* Ptr, SIZE_T NewSize, uint32 Alignment) override
	{
		if (Ptr && FPlatformMemory::PtrIsOSMalloc(Ptr))
		{
			return ::realloc(Ptr, NewSize);
		}

		return T::Realloc(Ptr, NewSize, Alignment);
	}

	virtual void* TryRealloc(void* Ptr, SIZE_T NewSize, uint32 Alignment) override
	{
		if (Ptr && FPlatformMemory::PtrIsOSMalloc(Ptr))
		{
			return ::realloc(Ptr, NewSize);
		}

		return T::TryRealloc(Ptr, NewSize, Alignment);
	}

	virtual void Free(void* Ptr) override
	{
		if (Ptr && FPlatformMemory::PtrIsOSMalloc(Ptr))
		{
			::free(Ptr);
		}
		else
		{
			T::Free(Ptr);
		}
	}
#endif
};

FMalloc* FApplePlatformMemory::BaseAllocator()
{
	static FMalloc* Instance = nullptr;
	if (Instance != nullptr)
	{
		return Instance;
	}

	const FPlatformMemoryStats MemStats = FApplePlatformMemory::GetStats();
#if ENABLE_LOW_LEVEL_MEM_TRACKER
	FLowLevelMemTracker::Get().SetProgramSize(MemStats.UsedPhysical);
#endif
	FPlatformMemory::ProgramSize = MemStats.UsedPhysical;

	SelectAllocatorToUse(MemStats);
    
    switch (AllocatorToUse)
    {
        case EMemoryAllocatorToUse::Ansi:
        {
            Instance = new FMallocAnsi();
            break;
        }

		case EMemoryAllocatorToUse::Mimalloc:
		{
			Instance = new FAppleMallocRouter<FMallocMimalloc>();
			break;
		}

		case EMemoryAllocatorToUse::Binned3:
		{
			Instance = new FAppleMallocRouter<FMallocBinned3>();
			break;
		}

        case EMemoryAllocatorToUse::Binned2:
        {
            Instance = new FAppleMallocRouter<FMallocBinned2>();
            break;
        }

        default:    // intentional fall-through
        case EMemoryAllocatorToUse::Binned:
        {
            // get free memory
			vm_statistics64 Stats{};
            mach_msg_type_number_t StatsSize = HOST_VM_INFO64_COUNT;
            host_statistics64(mach_host_self(), HOST_VM_INFO64, (host_info_t)&Stats, &StatsSize);
            // 1 << FMath::CeilLogTwo(MemoryConstants.TotalPhysical) should really be FMath::RoundUpToPowerOfTwo,
            // but that overflows to 0 when MemoryConstants.TotalPhysical is close to 4GB, since CeilLogTwo returns 32
            // this then causes the MemoryLimit to be 0 and crashing the app
            uint64 MemoryLimit = FMath::Min<uint64>( uint64(1) << FMath::CeilLogTwo((Stats.free_count + Stats.inactive_count) * GetConstants().PageSize), 0x100000000);

            // [RCL] 2017-03-06 FIXME: perhaps BinnedPageSize should be used here, but leaving this change to the Mac platform owner.
            Instance = new FMallocBinned((uint32)(GetConstants().PageSize&MAX_uint32), MemoryLimit);
        }
    }
	return Instance;
}

FPlatformMemoryStats FApplePlatformMemory::GetStats()
{
	const FPlatformMemoryConstants& MemoryConstants = FPlatformMemory::GetConstants();
	static FPlatformMemoryStats MemoryStats;
	
	// Gather platform memory stats for the host and the specific process.
	vm_statistics64 Stats{};
	task_vm_info ProcessStats{};
	mach_msg_type_number_t StatsSize = HOST_VM_INFO64_COUNT;
	mach_msg_type_number_t ProcessStatsSize = TASK_VM_INFO_COUNT;
	FMemory::Memset(&Stats, 0, sizeof(Stats));
	if (host_statistics64(mach_host_self(), HOST_VM_INFO64, (host_info_t)&Stats, &StatsSize))
	{
		UE_LOGF(LogTemp, Warning, "Failed to fetch vm statistics");
	}
	if (task_info(mach_task_self(), TASK_VM_INFO, (task_info_t)&ProcessStats, &ProcessStatsSize))
	{
		UE_LOGF(LogTemp, Warning, "Failed to fetch task-level vm statistics");
	}

	uint64_t FreeMem = 0;
#if PLATFORM_IOS
	FreeMem = os_proc_available_memory();
#else
    FreeMem = (Stats.free_count + Stats.inactive_count) * MemoryConstants.PageSize;
#endif

	MemoryStats.AvailablePhysical = FreeMem;

	// Calculate the number of available free pages on iOS/macOS. Apple considers "inactive_count" pages
	// as pages that the app says it doesn't need anymore and can be recycled/freed, but could be reactivated 
	// if the app does request those resources again.  Apple tries to maximize use of memory and 
	// considers "free" pages a waste of resources.
	MemoryStats.AvailableVirtual = (Stats.free_count + Stats.inactive_count) * MemoryConstants.PageSize;
	
	struct rusage_info_v5 rusage_payload;
		 
	int ret = proc_pid_rusage(getpid(),
							  RUSAGE_INFO_V5, // I.e., new RUSAGE_INFO_V5 as RUSAGE_INFO_V6 is unsupported on iOS 15.
							  (rusage_info_t *)&rusage_payload);
	NSCAssert(ret == 0, @"Could not get rusage: %i.", errno); // Look up in `man errno`.
			
	MemoryStats.UsedPhysical = rusage_payload.ri_phys_footprint;

	if(MemoryStats.UsedPhysical > MemoryStats.PeakUsedPhysical)
	{
		MemoryStats.PeakUsedPhysical = MemoryStats.UsedPhysical;
	}

	MemoryStats.UsedVirtual = ProcessStats.virtual_size;
	if(MemoryStats.UsedVirtual > MemoryStats.PeakUsedVirtual)
	{
		MemoryStats.PeakUsedVirtual = MemoryStats.UsedVirtual;
	}
	
	return MemoryStats;
}

const FPlatformMemoryConstants& FApplePlatformMemory::GetConstants()
{
	static FPlatformMemoryConstants MemoryConstants;
	
	if( MemoryConstants.TotalPhysical == 0 )
	{
		// Gather platform memory constants.
		int64 AvailablePhysical = 0;
		
#if PLATFORM_IOS && !WITH_IOS_SIMULATOR
		AvailablePhysical = os_proc_available_memory();
#else
		int Mib[] = {CTL_HW, HW_MEMSIZE};
		size_t Length = sizeof(int64);
		sysctl(Mib, 2, &AvailablePhysical, &Length, NULL, 0);
#endif
		
		MemoryConstants.TotalPhysical = AvailablePhysical;
		MemoryConstants.TotalVirtual = AvailablePhysical;	// Calculate true value below, but default to physical if vmstats call fails 
		MemoryConstants.PageSize = (uint32)vm_page_size;
		MemoryConstants.OsAllocationGranularity = (uint32)vm_page_size;
		MemoryConstants.BinnedPageSize = FMath::Max((SIZE_T)65536, (SIZE_T)vm_page_size);
		
		// macOS reports the correct amount of physical memory, however iOS reports a lower amount of 
		// actual physical memory. To work around this, we add 1Gb - 1b so it will be truncated 
		// correctly and will not affect macOS
		MemoryConstants.TotalPhysicalGB = ([NSProcessInfo processInfo].physicalMemory + (1024*1024*1024 - 1)) / 1024 / 1024 / 1024;
		MemoryConstants.AddressLimit = FPlatformMath::RoundUpToPowerOfTwo64(MemoryConstants.TotalPhysical);

		// Calculate total and available Virtual Memory
		mach_port_t HostPort = mach_host_self();
		mach_msg_type_number_t HostSize = sizeof(vm_statistics64_data_t) / sizeof(integer_t);

		// verify that actual device pagesize matches defined size in vm_page_size.h
		vm_size_t PageSize = 0;
		host_page_size(HostPort, &PageSize);
		ensure(vm_page_size == PageSize);

		vm_statistics64_data_t vm_stat;
		if (host_statistics64(HostPort, HOST_VM_INFO64, (host_info_t)&vm_stat, &HostSize) != KERN_SUCCESS)
		{
			UE_LOGF(LogTemp, Warning, "Failed to fetch vm statistics");
			return MemoryConstants;
		}

		uint64 VMemUsed = (vm_stat.active_count +
						  vm_stat.inactive_count +
						  vm_stat.wire_count) *
						 PageSize;
		uint64 VMemFree = vm_stat.free_count * PageSize;
		uint64 VMemTotal = VMemUsed + VMemFree;

		MemoryConstants.TotalVirtual = VMemTotal;
	}
	
	return MemoryConstants;
}

uint64 FApplePlatformMemory::GetMemoryUsedFast()
{
	mach_task_basic_info_data_t TaskInfo;
	mach_msg_type_number_t TaskInfoCount = MACH_TASK_BASIC_INFO_COUNT;
	task_info( mach_task_self(), MACH_TASK_BASIC_INFO, (task_info_t)&TaskInfo, &TaskInfoCount );
	return TaskInfo.resident_size;
}

bool FApplePlatformMemory::PageProtect(void* const Ptr, const SIZE_T Size, const bool bCanRead, const bool bCanWrite)
{
	int32 ProtectMode;
	if (bCanRead && bCanWrite)
	{
		ProtectMode = PROT_READ | PROT_WRITE;
	}
	else if (bCanRead)
	{
		ProtectMode = PROT_READ;
	}
	else if (bCanWrite)
	{
		ProtectMode = PROT_WRITE;
	}
	else
	{
		ProtectMode = PROT_NONE;
	}
	return mprotect(Ptr, Size, static_cast<int32>(ProtectMode)) == 0;
}

#ifndef MALLOC_LEAKDETECTION
	#define MALLOC_LEAKDETECTION 0
#endif

// check bookkeeping info against the passed in parameters in Debug and Development (the latter only in games and servers. also, only if leak detection is disabled, otherwise things are very slow)
#define UE_PLATFORM_SANITY_CHECK_OS_ALLOCATIONS			(UE_BUILD_DEBUG || (UE_BUILD_DEVELOPMENT && (UE_GAME || UE_SERVER) && !MALLOC_LEAKDETECTION))

/** This structure is stored in the page after each OS allocation and checks that its properties are valid on Free. Should be less than the page size (4096 on all platforms we support) */
struct FOSAllocationDescriptor
{
	enum class MagicType : uint64
	{
		Marker = 0xd0c233ccf493dfb0
	};

	/** Magic that makes sure that we are not passed a pointer somewhere into the middle of the allocation (and/or the structure wasn't stomped). */
	MagicType	Magic;

	/** This should include the descriptor itself. */
	void*		PointerToUnmap;

	/** This should include the total size of allocation, so after unmapping it everything is gone, including the descriptor */
	SIZE_T		SizeToUnmap;

	/** Debug info that makes sure that the correct size is preserved. */
	SIZE_T		OriginalSizeAsPassed;
};

void* _Nullable FApplePlatformMemory::BinnedAllocFromOS(SIZE_T Size)
{
	// Binned2 requires allocations to be BinnedPageSize-aligned. Simple mmap() does not guarantee this for recommended BinnedPageSize (64KB).
#if USE_MALLOC_BINNED2
	static SIZE_T OSPageSize = FPlatformMemory::GetConstants().PageSize;
	// guard against someone not passing size in whole pages
	SIZE_T SizeInWholePages = (Size % OSPageSize) ? (Size + OSPageSize - (Size % OSPageSize)) : Size;
	void* Pointer = nullptr;

	// Binned expects OS allocations to be BinnedPageSize-aligned, and that page is at least 64KB. mmap() alone cannot do this, so carve out the needed chunks.
	const SIZE_T ExpectedAlignment = FPlatformMemory::GetConstants().BinnedPageSize;
	// Descriptor is only used if we're sanity checking. However, #ifdef'ing its use would make the code more fragile. Size needs to be at least one page.
#if UE_PLATFORM_SANITY_CHECK_OS_ALLOCATIONS
	const SIZE_T DescriptorSize = OSPageSize;
#else
	const SIZE_T DescriptorSize = 0;
#endif

	SIZE_T ActualSizeMapped = SizeInWholePages + ExpectedAlignment;

	// the remainder of the map will be used for the descriptor, if any.
	// we always allocate at least one page more than needed
	void* PointerWeGotFromMMap = MmapWithTag(nullptr, ActualSizeMapped, PROT_READ | PROT_WRITE, MAP_PRIVATE | MAP_ANON, MmapTag::BinnedAllocFromOS, 0);
	// store these values, to unmap later

	Pointer = PointerWeGotFromMMap;
	if (Pointer == MAP_FAILED)
	{
		FPlatformMemory::OnOutOfMemory(ActualSizeMapped, ExpectedAlignment);
		// unreachable
	}

	SIZE_T Offset = (reinterpret_cast<SIZE_T>(Pointer) % ExpectedAlignment);

	// See if we need to unmap anything in the front. If the pointer happened to be aligned, we don't need to unmap anything.
	if (LIKELY(Offset != 0))
	{
		// figure out how much to unmap before the alignment.
		SIZE_T SizeToNextAlignedPointer = ExpectedAlignment - Offset;
		void* AlignedPointer = reinterpret_cast<void*>(reinterpret_cast<SIZE_T>(Pointer) + SizeToNextAlignedPointer);

		// unmap the part before
		if (munmap(Pointer, SizeToNextAlignedPointer) != 0)
		{
			FPlatformMemory::OnOutOfMemory(SizeToNextAlignedPointer, ExpectedAlignment);
			// unreachable
		}

		// account for reduced mmaped size
		ActualSizeMapped -= SizeToNextAlignedPointer;

		// now, make it appear as if we initially got the allocation right
		Pointer = AlignedPointer;
	}

	// at this point, Pointer is aligned at the expected alignment - either we lucked out on the initial allocation
	// or we already got rid of the extra memory that was allocated in the front.
	checkf((reinterpret_cast<SIZE_T>(Pointer) % ExpectedAlignment) == 0, TEXT("BinnedAllocFromOS(): Internal error: did not align the pointer as expected."));

	// Now unmap the tail only, if any, but leave just enough space for the descriptor
	void* TailPtr = reinterpret_cast<void*>(reinterpret_cast<SIZE_T>(Pointer) + SizeInWholePages + DescriptorSize);
	SSIZE_T TailSize = ActualSizeMapped - SizeInWholePages - DescriptorSize;

	if (LIKELY(TailSize > 0))
	{
		if (munmap(TailPtr, TailSize) != 0)
		{
			FPlatformMemory::OnOutOfMemory(TailSize, ExpectedAlignment);
			// unreachable
		}
	}

	// we're done with this allocation, fill in the descriptor with the info
	if (LIKELY(DescriptorSize > 0))
	{
		FOSAllocationDescriptor* AllocDescriptor = reinterpret_cast<FOSAllocationDescriptor*>(reinterpret_cast<SIZE_T>(Pointer) + Size);
		AllocDescriptor->Magic = FOSAllocationDescriptor::MagicType::Marker;
		AllocDescriptor->PointerToUnmap = Pointer;
		AllocDescriptor->SizeToUnmap = SizeInWholePages + DescriptorSize;
		AllocDescriptor->OriginalSizeAsPassed = Size;
	}

	LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelAlloc(ELLMTracker::Platform, Pointer, Size));
	return Pointer;
#else
	void* Ptr = MmapWithTag(nullptr, Size, PROT_READ | PROT_WRITE, MAP_PRIVATE | MAP_ANON, MmapTag::BinnedAllocFromOS, 0);
	if (Ptr == (void*)-1)
	{
		UE_LOGF(LogTemp, Warning, "mmap failure allocating %zu, error code: %d", Size, errno);
		Ptr = nullptr;
	}
	LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelAlloc(ELLMTracker::Platform, Ptr, Size));
	return Ptr;
#endif // USE_MALLOC_BINNED2
}

void FApplePlatformMemory::BinnedFreeToOS(void* Ptr, SIZE_T Size)
{
	// Binned2 requires allocations to be BinnedPageSize-aligned. Simple mmap() does not guarantee this for recommended BinnedPageSize (64KB).
#if USE_MALLOC_BINNED2
	LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelFree(ELLMTracker::Platform, Ptr));
	// guard against someone not passing size in whole pages
	static SIZE_T OSPageSize = FPlatformMemory::GetConstants().PageSize;
	SIZE_T SizeInWholePages = (Size % OSPageSize) ? (Size + OSPageSize - (Size % OSPageSize)) : Size;

	#if (UE_PLATFORM_SANITY_CHECK_OS_ALLOCATIONS)
	{
		const SIZE_T DescriptorSize = OSPageSize;

		FOSAllocationDescriptor* AllocDescriptor = reinterpret_cast<FOSAllocationDescriptor*>(reinterpret_cast<SIZE_T>(Ptr) + Size);
		if (UNLIKELY(AllocDescriptor->Magic != FOSAllocationDescriptor::MagicType::Marker))
		{
			UE_LOGF(LogHAL, Fatal, "BinnedFreeToOS() has been passed an address %p (size %llu) not allocated through it.", Ptr, (uint64)Size);
			// unreachable
			return;
		}

		void* PointerToUnmap = AllocDescriptor->PointerToUnmap;
		SIZE_T SizeToUnmap = AllocDescriptor->SizeToUnmap;

		// this check only makes sense when we're not reducing number of maps, since the pointer will have to be different.
		if (UNLIKELY(PointerToUnmap != Ptr || SizeToUnmap != SizeInWholePages + DescriptorSize))
		{
			UE_LOGF(LogHAL, Fatal, "BinnedFreeToOS(): info mismatch: descriptor ptr: %p, size %llu, but our pointer is %p and size %llu.", PointerToUnmap, SizeToUnmap, AllocDescriptor, (uint64)(SizeInWholePages + DescriptorSize));
			// unreachable
			return;
		}

		if (UNLIKELY(AllocDescriptor->OriginalSizeAsPassed != Size))
		{
			UE_LOGF(LogHAL, Fatal, "BinnedFreeToOS(): info mismatch: descriptor original size %llu, our size is %llu for pointer %p", AllocDescriptor->OriginalSizeAsPassed, Size, Ptr);
			// unreachable
			return;
		}

		AllocDescriptor = nullptr;	// just so no one touches it

		if (UNLIKELY(munmap(PointerToUnmap, SizeToUnmap) != 0))
		{
			FPlatformMemory::OnOutOfMemory(SizeToUnmap, 0);
			// unreachable
		}
	}
	#else
	{
		if (UNLIKELY(munmap(Ptr, SizeInWholePages) != 0))
		{
			FPlatformMemory::OnOutOfMemory(SizeInWholePages, 0);
			// unreachable
		}
	}
	#endif
#else
	LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelFree(ELLMTracker::Platform, Ptr));
	if (munmap(Ptr, Size) != 0)
	{
		const int ErrNo = errno;
		UE_LOGF(LogHAL, Fatal, "munmap(addr=%p, len=%llu) failed with errno = %d (%ls)", Ptr, Size,
			ErrNo, StringCast< TCHAR >(strerror(ErrNo)).Get());
	}
#endif // USE_MALLOC_BINNED2
}

bool FApplePlatformMemory::PtrIsOSMalloc( void* Ptr)
{
	return malloc_zone_from_ptr(Ptr) != nullptr;
}

bool FApplePlatformMemory::IsNanoMallocAvailable()
{
	return (NanoRegionStart != 0) && (NanoRegionEnd != 0);
}

bool FApplePlatformMemory::PtrIsFromNanoMalloc( void* Ptr)
{
	return IsNanoMallocAvailable() && ((uintptr_t) Ptr >= NanoRegionStart && (uintptr_t) Ptr < NanoRegionEnd);
}

size_t FApplePlatformMemory::FPlatformVirtualMemoryBlock::GetVirtualSizeAlignment()
{
	static SIZE_T OSPageSize = FPlatformMemory::GetConstants().PageSize;
	return OSPageSize;
}

size_t FApplePlatformMemory::FPlatformVirtualMemoryBlock::GetCommitAlignment()
{
	static SIZE_T OSPageSize = FPlatformMemory::GetConstants().PageSize;
	return OSPageSize;
}

FApplePlatformMemory::FPlatformVirtualMemoryBlock FApplePlatformMemory::FPlatformVirtualMemoryBlock::AllocateVirtual(size_t InSize, size_t InAlignment)
{
	FPlatformVirtualMemoryBlock Result;
	InSize = Align(InSize, GetVirtualSizeAlignment());
	Result.VMSizeDivVirtualSizeAlignment = InSize / GetVirtualSizeAlignment();
	size_t Alignment = FMath::Max(InAlignment, GetVirtualSizeAlignment());
	check(Alignment <= GetVirtualSizeAlignment());

	Result.Ptr = MmapWithTag(nullptr, Result.GetActualSize(), PROT_READ | PROT_WRITE, MAP_PRIVATE | MAP_ANON | MAP_NORESERVE, MmapTag::AllocateVirtualMemoryBlock, 0);
	if (!LIKELY(Result.Ptr != MAP_FAILED))
	{
		FPlatformMemory::OnOutOfMemory(Result.GetActualSize(), InAlignment);
	}
	check(Result.Ptr && IsAligned(Result.Ptr, Alignment));
	return Result;
}



void FApplePlatformMemory::FPlatformVirtualMemoryBlock::FreeVirtual()
{
	if (Ptr)
	{
		check(GetActualSize() > 0);
		if (munmap(Ptr, GetActualSize()) != 0)
		{
			// we can ran out of VMAs here
			FPlatformMemory::OnOutOfMemory(GetActualSize(), 0);
			// unreachable
		}
		Ptr = nullptr;
		VMSizeDivVirtualSizeAlignment = 0;
	}
}

void FApplePlatformMemory::FPlatformVirtualMemoryBlock::Commit(size_t InOffset, size_t InSize)
{
	check(IsAligned(InOffset, GetCommitAlignment()) && IsAligned(InSize, GetCommitAlignment()));
	check(InOffset >= 0 && InSize >= 0 && InOffset + InSize <= GetActualSize() && Ptr);
	madvise(((uint8*)Ptr) + InOffset, InSize, MADV_FREE_REUSE);
}

void FApplePlatformMemory::FPlatformVirtualMemoryBlock::Decommit(size_t InOffset, size_t InSize)
{
	check(IsAligned(InOffset, GetCommitAlignment()) && IsAligned(InSize, GetCommitAlignment()));
	check(InOffset >= 0 && InSize >= 0 && InOffset + InSize <= GetActualSize() && Ptr);
	if (madvise(((uint8*)Ptr) + InOffset, InSize, MADV_FREE_REUSABLE) != 0)
	{
		// we can ran out of VMAs here too!
		FPlatformMemory::OnOutOfMemory(InSize, 0);
	}
}



/**
 * LLM uses these low level functions (LLMAlloc and LLMFree) to allocate memory. It grabs
 * the function pointers by calling FPlatformMemory::GetLLMAllocFunctions. If these functions
 * are not implemented GetLLMAllocFunctions should return false and LLM will be disabled.
 */

#if ENABLE_LOW_LEVEL_MEM_TRACKER

int64 LLMMallocTotal = 0;

void* LLMAlloc(size_t Size)
{
    void* Ptr = MmapWithTag(nullptr, Size, PROT_READ | PROT_WRITE, MAP_PRIVATE | MAP_ANON, MmapTag::LLMAlloc, 0);
	if (Ptr == MAP_FAILED)
	{
		const int ErrNo = errno;
		UE_LOGF(LogHAL, Error, "mmap(Size=%llu) failed with errno = %d (%ls)", Size,
			ErrNo, StringCast< TCHAR >(strerror(ErrNo)).Get());
	}
	else
	{
		LLMMallocTotal += Size;
	}
    
    return Ptr;
}

void LLMFree(void* Addr, size_t Size)
{
    LLMMallocTotal -= Size;
    if (Addr != nullptr && munmap(Addr, Size) != 0)
    {
        const int ErrNo = errno;
        UE_LOGF(LogHAL, Fatal, "munmap(addr=%p, len=%llu) failed with errno = %d (%ls)", Addr, Size,
               ErrNo, StringCast< TCHAR >(strerror(ErrNo)).Get());
    }
}

#endif

bool FApplePlatformMemory::GetLLMAllocFunctions(void*_Nonnull(*_Nonnull&OutAllocFunction)(size_t), void(*_Nonnull&OutFreeFunction)(void*, size_t), int32& OutAlignment)
{
#if ENABLE_LOW_LEVEL_MEM_TRACKER
    OutAllocFunction = LLMAlloc;
    OutFreeFunction = LLMFree;
    OutAlignment = vm_page_size;
    return true;
#else
    return false;
#endif
}

#if APPLE_SUPPORT_INSTRUMENTED_ALLOCS

#if USE_APPLE_SUPPORT_INSTRUMENTED_ALLOCS

// https://github.com/apple-oss-distributions/CF/blob/main/CFRuntime.c#L84-L92
enum class CFTraceEvent : int32_t
{
	ObjectRetained = 12,
	ObjectReleased = 13,
	Malloc = 16,
	_Free = 19,
	Zombie = 21,
	VMalloc = 23,
	_Free2 = 26, // ?
	Retain = 28,
	Released = 29,
	_Free3 = 30, // ?
};
extern "C" void __CFRecordAllocationEvent(int32_t CFTraceType, void* Ptr, int64_t Size, uint64_t Data, const char* Name);
extern "C" void __CFSetLastAllocationEventName(void* Ptr, const char* Name);

// https://github.com/apple-oss-distributions/libmalloc/blob/main/private/stack_logging.h#L71
enum class MallocLogger : uint32_t
{
	Free = 0,
	Generic = 1,
	Alloc = 2,
	Dealloc = 4,
	FlagZone = 8, // ?
	VMAlloc = 16,
	VMDealloc = 32,
	FlagCleared = 64, // ?
	FileMapOrSharedMem = 128
};
typedef void(FnMallocLogger)(uint32_t MallocLoggerType, uintptr_t Arg1, uintptr_t Arg2, uintptr_t Arg3, uintptr_t Result, uint32_t NumFramesToSkip);
extern "C" FnMallocLogger *malloc_logger;

void FApplePlatformMemory::OnLowLevelMemory_Alloc(void const* Pointer, uint64 Size, uint64 Tag)
{
#if ENABLE_LOW_LEVEL_MEM_TRACKER
	const char* EventName = FLowLevelMemTracker::IsEnabled() ? LLMGetTagNameANSI((ELLMTag)Tag) : "UE";

	// mmap will already call logger internally
	// see https://github.com/apple-oss-distributions/xnu/blob/main/libsyscall/wrappers/unix03/mmap.c#L72
	if (Tag == (uint64)ELLMTracker::Platform)
	{
		__CFSetLastAllocationEventName((void*)Pointer, EventName);
		return;
	}
#else
	const char* EventName = "UE";
#endif // ENABLE_LOW_LEVEL_MEM_TRACKER

	if (malloc_logger)
	{
		malloc_logger((uint32_t)MallocLogger::Alloc, 0, (uintptr_t)Size, 0, (uintptr_t)Pointer, 0);
		__CFSetLastAllocationEventName((void*)Pointer, EventName);
	}
}

void FApplePlatformMemory::OnLowLevelMemory_Free(void const* Pointer, uint64 Size, uint64 Tag)
{
#if ENABLE_LOW_LEVEL_MEM_TRACKER
	// munmap will already call logger internally
	if (Tag == (uint64)ELLMTracker::Platform)
	{
		return;
	}
#endif // ENABLE_LOW_LEVEL_MEM_TRACKER

	if (malloc_logger)
	{
		malloc_logger((uint32_t)MallocLogger::Dealloc, 0, (uintptr_t)Pointer, 0, 0, 0);
	}
}

#else

void FApplePlatformMemory::OnLowLevelMemory_Alloc(void const* Pointer, uint64 Size, uint64 Tag)
{
}

void FApplePlatformMemory::OnLowLevelMemory_Free(void const* Pointer, uint64 Size, uint64 Tag)
{
}

#endif // USE_APPLE_SUPPORT_INSTRUMENTED_ALLOCS

#endif // APPLE_SUPPORT_INSTRUMENTED_ALLOCS

bool FApplePlatformMemory::CanOverallocateVirtualMemory()
{
#if PLATFORM_IOS || PLATFORM_TVOS
	static bool bHasExtendedVirtualAddressingEntitlement = FIOSPlatformMisc::IsEntitlementEnabled("com.apple.developer.kernel.extended-virtual-addressing");
	return bHasExtendedVirtualAddressingEntitlement;
#else
	return true;	// 64 bit Mac process can allocate ~18 exabytes of addressable space
#endif
}

NS_ASSUME_NONNULL_END
