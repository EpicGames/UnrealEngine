// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ApplePlatformMisc.mm: iOS implementations of misc functions
=============================================================================*/

#include "Apple/ApplePlatformMisc.h"
#if PLATFORM_MAC
#include "Mac/MacPlatformMiscEx.h"
#elif PLATFORM_IOS
#include "IOS/IOSSystemIncludes.h"
#endif
#include "HAL/ExceptionHandling.h"
#include "Misc/SecureHash.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/Guid.h"
#include "Apple/ApplePlatformDebugEvents.h"
#include "Apple/ApplePlatformCrashContext.h"
#include "FramePro/FrameProProfiler.h"
#include "CoreGlobals.h"
#include "Async/TaskGraphInterfaces.h"
#include "Misc/CoreDelegates.h"

#include <os/log.h>
#import <Network/Network.h>

DEFINE_LOG_CATEGORY_STATIC(LogApplePlatformMisc, Log, All);

#if !UE_BUILD_SHIPPING

bool FApplePlatformMisc::IsDebuggerPresent()
{
	// Based on http://developer.apple.com/library/mac/#qa/qa1361/_index.html

	if (GIgnoreDebugger)
	{
		return false;
	}

	struct kinfo_proc Info;
	int32 Mib[] = { CTL_KERN, KERN_PROC, KERN_PROC_PID, getpid() };
	SIZE_T Size = sizeof(Info);

	sysctl(Mib, sizeof(Mib) / sizeof(*Mib), &Info, &Size, NULL, 0);

	return (Info.kp_proc.p_flag & P_TRACED) != 0;
}

#endif // !UE_BUILD_SHIPPING

void FApplePlatformMisc::MemoryBarrier()
{
	__sync_synchronize();
}

FString FApplePlatformMisc::GetEnvironmentVariable(const TCHAR* VariableName)
{
	// Replace hyphens with underscores. Some legacy UE environment variables (eg. UE-SharedDataCachePath) are in widespread
	// usage in their hyphenated form, but are not normally valid shell variables.
	FString FixedVariableName = VariableName;
	FixedVariableName.ReplaceInline(TEXT("-"), TEXT("_"));
	ANSICHAR *AnsiResult = getenv(TCHAR_TO_ANSI(*FixedVariableName));
	if (AnsiResult)
	{
		return ANSI_TO_TCHAR(AnsiResult);
	}
	else
	{
		return FString();
	}
}

void FApplePlatformMisc::LocalPrint(const TCHAR* Message)
{
#if !UE_BUILD_SHIPPING || ENABLE_PGO_PROFILE
	// Use os_log with %{public}s so messages are visible in Console.app
	// without Xcode attached. NSLog is redacted on iOS 26+ unified logging.
	os_log(OS_LOG_DEFAULT, "[UE] %{public}s", TCHAR_TO_UTF8(Message));
#else
	// This only goes to the system console.
	// Private by default, would be visible in Xcode.
	os_log(OS_LOG_DEFAULT, "[UE] %s", TCHAR_TO_UTF8(Message));
#endif
}

const TCHAR* FApplePlatformMisc::GetSystemErrorMessage(TCHAR* OutBuffer, int32 BufferCount, int32 Error)
{
	check(OutBuffer && BufferCount);
	*OutBuffer = TEXT('\0');
	if (Error == 0)
	{
		Error = errno;
	}
	char* ErrorBuffer = (char*)alloca(BufferCount);
	if (strerror_r(Error, ErrorBuffer, 1024) == 0)
	{
		FCString::Strncpy(OutBuffer, UTF8_TO_TCHAR((const ANSICHAR*)ErrorBuffer), BufferCount);
	}
	else
	{
		*OutBuffer = TEXT('\0');
	}
	return OutBuffer;
}

uint32 FApplePlatformMisc::GetLastError()
{
	return (uint32)errno;
}

FString FApplePlatformMisc::GetDefaultLocale()
{
	CFLocaleRef Locale = CFLocaleCopyCurrent();
	CFStringRef LangCodeStr = (CFStringRef)CFLocaleGetValue(Locale, kCFLocaleLanguageCode);
	FString LangCode((__bridge NSString*)LangCodeStr);
	CFStringRef CountryCodeStr = (CFStringRef)CFLocaleGetValue(Locale, kCFLocaleCountryCode);
	FString CountryCode((__bridge NSString*)CountryCodeStr);
	CFRelease(Locale);

	return CountryCode.IsEmpty() ? LangCode : FString::Printf(TEXT("%s-%s"), *LangCode, *CountryCode);
}

FString FApplePlatformMisc::GetDefaultLanguage()
{
	CFArrayRef Languages = CFLocaleCopyPreferredLanguages();
	CFStringRef LangCodeStr = (CFStringRef)CFArrayGetValueAtIndex(Languages, 0);
	FString LangCode((__bridge NSString*)LangCodeStr);
	CFRelease(Languages);

	return LangCode;
}

int32 FApplePlatformMisc::NumberOfCores()
{
	// cache the number of cores
	static int32 NumberOfCores = -1;
	if (NumberOfCores == -1)
	{
		SIZE_T Size = sizeof(int32);
		if (sysctlbyname("hw.ncpu", &NumberOfCores, &Size, nullptr, 0) != 0)
		{
			NumberOfCores = 1;
		}
	}
	return NumberOfCores;
}

void FApplePlatformMisc::CreateGuid(FGuid& Result)
{
	uuid_t UUID;
	uuid_generate(UUID);
	
	uint32* Values = (uint32*)(&UUID[0]);
	Result[0] = Values[0];
	Result[1] = Values[1];
	Result[2] = Values[2];
	Result[3] = Values[3];
}

void* FApplePlatformMisc::CreateAutoreleasePool()
{
	return [[NSAutoreleasePool alloc] init];
}

void FApplePlatformMisc::ReleaseAutoreleasePool(void *Pool)
{
	[(NSAutoreleasePool*)Pool release];
}

struct FFontHeader
{
	int32 Version;
	uint16 NumTables;
	uint16 SearchRange;
	uint16 EntrySelector;
	uint16 RangeShift;
};

struct FFontTableEntry
{
	uint32 Tag;
	uint32 CheckSum;
	uint32 Offset;
	uint32 Length;
};


static uint32 CalcTableCheckSum(const uint32 *Table, uint32 NumberOfBytesInTable)
{
	uint32 Sum = 0;
	uint32 NumLongs = (NumberOfBytesInTable + 3) / 4;
	while (NumLongs-- > 0)
	{
		Sum += CFSwapInt32HostToBig(*Table++);
	}
	return Sum;
}

static uint32 CalcTableDataRefCheckSum(CFDataRef DataRef)
{
	const uint32 *DataBuff = (const uint32 *)CFDataGetBytePtr(DataRef);
	uint32 DataLength = (uint32)CFDataGetLength(DataRef);
	return CalcTableCheckSum(DataBuff, DataLength);
}

/**
 * In order to get a system font from IOS we need to build one from the data we can gather from a CGFontRef
 * @param InFontName - The name of the system font we are seeking to load.
 * @param OutBytes - The data we have built for the font.
 */
void GetBytesForFont(const NSString* InFontName, OUT TArray<uint8>& OutBytes)
{
	CGFontRef cgFont = CGFontCreateWithFontName((CFStringRef)InFontName);

	if (cgFont)
	{
		CFRetain(cgFont);

		// Gather information on the font tags
		CFArrayRef Tags = CGFontCopyTableTags(cgFont);
		int TableCount = CFArrayGetCount(Tags);

		// Collate the table sizes
		TArray<size_t> TableSizes;

		bool bContainsCFFTable = false;

		size_t TotalSize = sizeof(FFontHeader)+sizeof(FFontTableEntry)* TableCount;
		for (int TableIndex = 0; TableIndex < TableCount; ++TableIndex)
		{
			size_t TableSize = 0;
			
			uint64 aTag = (uint64)CFArrayGetValueAtIndex(Tags, TableIndex);
			if (aTag == 'CFF ' && !bContainsCFFTable)
			{
				bContainsCFFTable = true;
			}

			CFDataRef TableDataRef = CGFontCopyTableForTag(cgFont, aTag);
			if (TableDataRef != NULL)
			{
				TableSize = CFDataGetLength(TableDataRef);
				CFRelease(TableDataRef);
			}

			TotalSize += (TableSize + 3) & ~3;
			TableSizes.Add( TableSize );
		}

		OutBytes.Reserve( TotalSize );
		OutBytes.AddZeroed( TotalSize );

		// Start copying the table data into our buffer
		uint8* DataStart = OutBytes.GetData();
		uint8* DataPtr = DataStart;

		// Compute font header entries
		uint16 EntrySelector = 0;
		uint16 SearchRange = 1;
		while (SearchRange < TableCount >> 1)
		{
			EntrySelector++;
			SearchRange <<= 1;
		}
		SearchRange <<= 4;

		uint16 RangeShift = (uint16)((TableCount << 4) - SearchRange);

		// Write font header (also called sfnt header, offset subtable)
		FFontHeader* OffsetTable = (FFontHeader*)DataPtr;

		// OpenType Font contains CFF Table use 'OTTO' as version, and with .otf extension
		// otherwise 0001 0000
		OffsetTable->Version = bContainsCFFTable ? 'OTTO' : CFSwapInt16HostToBig(1);
		OffsetTable->NumTables = CFSwapInt16HostToBig((uint16)TableCount);
		OffsetTable->SearchRange = CFSwapInt16HostToBig((uint16)SearchRange);
		OffsetTable->EntrySelector = CFSwapInt16HostToBig((uint16)EntrySelector);
		OffsetTable->RangeShift = CFSwapInt16HostToBig((uint16)RangeShift);

		DataPtr += sizeof(FFontHeader);

		// Write tables
		FFontTableEntry* CurrentTableEntry = (FFontTableEntry*)DataPtr;
		DataPtr += sizeof(FFontTableEntry) * TableCount;

		for (int TableIndex = 0; TableIndex < TableCount; ++TableIndex)
		{
			uint64 aTag = (uint64)CFArrayGetValueAtIndex(Tags, TableIndex);
			CFDataRef TableDataRef = CGFontCopyTableForTag(cgFont, aTag);
			uint32 TableSize = CFDataGetLength(TableDataRef);

			FMemory::Memcpy(DataPtr, CFDataGetBytePtr(TableDataRef), TableSize);

			CurrentTableEntry->Tag = CFSwapInt32HostToBig((uint32_t)aTag);
			CurrentTableEntry->CheckSum = CFSwapInt32HostToBig(CalcTableCheckSum((uint32 *)DataPtr, TableSize));

			uint32 Offset = DataPtr - DataStart;
			CurrentTableEntry->Offset = CFSwapInt32HostToBig((uint32)Offset);
			CurrentTableEntry->Length = CFSwapInt32HostToBig((uint32)TableSize);

			DataPtr += (TableSize + 3) & ~3;
			++CurrentTableEntry;

			CFRelease(TableDataRef);
		}

		CFRelease(cgFont);
	}
}


TArray<uint8> FApplePlatformMisc::GetSystemFontBytes()
{
#if PLATFORM_MAC
	// Gather some details about the system font
	uint32 SystemFontSize = (uint32)[NSFont systemFontSize];
	NSString* SystemFontName = [NSFont systemFontOfSize:SystemFontSize].fontName;
#elif PLATFORM_TVOS
	NSString* SystemFontName = [UIFont preferredFontForTextStyle:UIFontTextStyleBody].fontName;
#else
	// Gather some details about the system font
	uint32 SystemFontSize = (uint32)[UIFont systemFontSize];
	NSString* SystemFontName = [UIFont systemFontOfSize:SystemFontSize].fontName;
#endif

	TArray<uint8> FontBytes;
	GetBytesForFont(SystemFontName, FontBytes);

	return FontBytes;
}

FString FApplePlatformMisc::GetLocalCurrencyCode()
{
	return FString([[NSLocale currentLocale] objectForKey:NSLocaleCurrencyCode]);
}

FString FApplePlatformMisc::GetLocalCurrencySymbol()
{
	return FString([[NSLocale currentLocale] objectForKey:NSLocaleCurrencySymbol]);
}

bool FApplePlatformMisc::IsOSAtLeastVersion(const uint32 MacOSVersion[3], const uint32 IOSVersion[3], const uint32 TVOSVersion[3])
{
	NSOperatingSystemVersion CurrentSystemVersion =
#if PLATFORM_MAC
	FMacPlatformMiscEx::GetNSOperatingSystemVersion();
#else
	[NSProcessInfo processInfo].operatingSystemVersion;
#endif

	static const uint32 OSVersion[3] = { (uint32)CurrentSystemVersion.majorVersion, (uint32)CurrentSystemVersion.minorVersion, (uint32)CurrentSystemVersion.patchVersion };
	const uint32* VersionToCompare = PLATFORM_MAC ? MacOSVersion : (PLATFORM_IOS ? IOSVersion : TVOSVersion);

	for (uint32 Index = 0; Index < 3; Index++)
	{
		if (OSVersion[Index] < VersionToCompare[Index])
		{
			return false;
		}
		else if (OSVersion[Index] > VersionToCompare[Index])
		{
			return true;
		}
	}
	return true;
}

void FApplePlatformMisc::SetEnvironmentVar(const TCHAR* InVariableName, const TCHAR* Value)
{
	FString VariableName = InVariableName;
	VariableName.ReplaceInline(TEXT("-"), TEXT("_"));
	if (Value == NULL || Value[0] == TEXT('\0'))
	{
		unsetenv(TCHAR_TO_ANSI(*VariableName));
	}
	else
	{
		setenv(TCHAR_TO_ANSI(*VariableName), TCHAR_TO_ANSI(Value), 1);
	}
}

#if APPLE_PROFILING_ENABLED

void FApplePlatformMisc::BeginNamedEvent(const struct FColor& Color, const TCHAR* Text)
{
	FGenericPlatformMisc::BeginNamedEvent(Color, Text);
#if !FRAMEPRO_ENABLED && APPLE_PROFILING_ENABLED
	FApplePlatformDebugEvents::BeginNamedEvent(Color, Text);
#endif // FRAMEPRO_ENABLED
}

void FApplePlatformMisc::BeginNamedEvent(const struct FColor& Color, const ANSICHAR* Text)
{
	FGenericPlatformMisc::BeginNamedEvent(Color, Text);
#if !FRAMEPRO_ENABLED && APPLE_PROFILING_ENABLED
	FApplePlatformDebugEvents::BeginNamedEvent(Color, Text);
#endif // FRAMEPRO_ENABLED
}

void FApplePlatformMisc::EndNamedEvent()
{
	FGenericPlatformMisc::EndNamedEvent();
#if !FRAMEPRO_ENABLED && APPLE_PROFILING_ENABLED
	FApplePlatformDebugEvents::EndNamedEvent();
#endif // FRAMEPRO_ENABLED
}

#endif // APPLE_PROFILING_ENABLED

namespace UE::FApplePlatformMisc
{
	// Used to monitor network state. Replaces the deprecated SCNetworkReachability API.
	// Note: defined here to not force objcisims into the header which is included on non-objc platforms
	static nw_path_monitor_t PathMonitor = nullptr;
	static nw_path_t CachedNetworkPath = nullptr;
	static ENetworkConnectionType LastReportedNetworkType = ENetworkConnectionType::Unknown;
	static ::FApplePlatformMisc::FNetworkConnectionCharacteristics CachedConnectionCharacteristics;

	static TMulticastDelegate<void(::FApplePlatformMisc::FNetworkConnectionCharacteristics), FDefaultTSDelegateUserPolicy> OnNetworkConnectionCharacteristicsChanged;

	// This is called from a whatever task graph thread picks.
	static void NotifyAboutNetworkConnectionCharacteristicsChange(nw_path_t InPath)
	{
		::FApplePlatformMisc::FNetworkConnectionCharacteristics Details;
		
		Details.bSupportsDNS   = nw_path_has_dns(InPath);
		Details.bSupportsIPv4  = nw_path_has_ipv4(InPath);
		Details.bSupportsIPv6  = nw_path_has_ipv6(InPath);
		Details.bIsConstrained = nw_path_is_constrained(InPath);
		Details.bIsExpensive   = nw_path_is_expensive(InPath);
		
		// Path's reference count was increased to guarantee it lives
		// til this moment of time. Refer to nw_path_monitor_set_update_handler
		// callback below. However, we just used it, so:
		nw_release(InPath);

		CachedConnectionCharacteristics = Details;
		OnNetworkConnectionCharacteristicsChanged.Broadcast(Details);
	}

	static ENetworkConnectionType CheckNetworkConnectionType()
	{
		nw_path_t MyPath = UE::FApplePlatformMisc::CachedNetworkPath;
		if (MyPath == nullptr)
		{
			UE_LOGF(LogInit, Warning, "Network path currently NOT set");
			return ENetworkConnectionType::Unknown;
		}
		
		nw_path_status_t PathStatus = nw_path_get_status(MyPath);
		if (PathStatus != nw_path_status_satisfied)
		{
			if (PathStatus == nw_path_status_invalid)
			{
				UE_LOGF(LogInit, Warning, "Network path is invalid");
				return ENetworkConnectionType::None;
			}
			
			// Basically, if there's _no_ network interfaces available, assume it's in airplane mode.
			// Note: if you have wifi enabled - but airplane mode also enabled, that will NOT be detected
			//       as airplane mode as you still have a valid network path.
			__block bool HasAnInterface = false;
			nw_path_enumerate_interfaces(MyPath, (nw_path_enumerate_interfaces_block_t) ^ (nw_interface_t Interface)
			{
				const char* Name = nw_interface_get_name(Interface);
				// UE_LOGF(LogIOS, Log, "NW path monitor available interface: %ls", *FString(Name));
				HasAnInterface = true;
				return false;
			});
			
			if (!HasAnInterface)
			{
				return ENetworkConnectionType::AirplaneMode;
			}
			return ENetworkConnectionType::None;
		}
		else
		{
			// If satisfied detect the type of connection or maybe use isExpensive???
			bool bHasActiveWiFiConnection = nw_path_uses_interface_type(MyPath, nw_interface_type_wifi);
			bool bHasActiveCellConnection = nw_path_uses_interface_type(MyPath, nw_interface_type_cellular);
			bool bHasActiveWiredConnection = nw_path_uses_interface_type(MyPath, nw_interface_type_wired);

			if (bHasActiveWiFiConnection)
			{
				return ENetworkConnectionType::WiFi;
			}
			else if (bHasActiveCellConnection)
			{
				return ENetworkConnectionType::Cell;
			}
			else if (bHasActiveWiredConnection)
			{
				return ENetworkConnectionType::Ethernet;
			}
			else
			{
				// For now, mw_interface_type_loopback and nw_interface_type_other return "Unknown"
				return ENetworkConnectionType::Unknown;
			}
		}
	}
} // UE::FApplePlatformMisc

ENetworkConnectionType FApplePlatformMisc::GetNetworkConnectionType()
{
	static TOptional<ENetworkConnectionType> ConnectionType = {};
	static double LastCheckTime = 0;

	const double CurrentTime = FPlatformTime::Seconds();
	const double CheckInterval = 0.2;

	if (!ConnectionType.IsSet() || CurrentTime >= LastCheckTime + CheckInterval)
	{
		ConnectionType = UE::FApplePlatformMisc::CheckNetworkConnectionType();
		LastCheckTime = CurrentTime;
	}

	ensure(ConnectionType.IsSet());
	return ConnectionType.GetValue();
}

bool FApplePlatformMisc::IsDataSavingNetworkConnection()
{
	return GetNetworkConnectionCharacteristics().bIsConstrained;
}

FApplePlatformMisc::FNetworkConnectionCharacteristics FApplePlatformMisc::GetNetworkConnectionCharacteristics()
{
	return UE::FApplePlatformMisc::CachedConnectionCharacteristics;
}

bool FApplePlatformMisc::HasActiveWiFiConnection()
{
	return GetNetworkConnectionType() == ENetworkConnectionType::WiFi;
}

TMulticastDelegateRegistration<void(FApplePlatformMisc::FNetworkConnectionCharacteristics), FDefaultTSDelegateUserPolicy>& FApplePlatformMisc::OnNetworkConnectionCharacteristicsChanged()
{
	return UE::FApplePlatformMisc::OnNetworkConnectionCharacteristicsChanged;
}

void FApplePlatformMisc::PlatformInit()
{
	// start monitoring network status
	UE::FApplePlatformMisc::PathMonitor = nw_path_monitor_create();
	if (UE::FApplePlatformMisc::PathMonitor == nullptr)
	{
		UE_LOGF(LogApplePlatformMisc, Warning, "Failed to create nw_path_monitor for network monitoring");
	}
	else
	{
		nw_path_monitor_set_update_handler(
			UE::FApplePlatformMisc::PathMonitor,
			^(nw_path_t InPath)
			{
				bool bIsConstrained = nw_path_is_constrained(InPath);
				bool bIsExpensive   = nw_path_is_expensive(InPath);

				UE_LOGF(LogApplePlatformMisc, Log, "Network path changed (IsConstrained=%d, IsExpensive=%d)", bIsConstrained, bIsExpensive);

				// cache the path we get
				if (UE::FApplePlatformMisc::CachedNetworkPath != nullptr)
				{
					nw_release(UE::FApplePlatformMisc::CachedNetworkPath);
				}
				UE::FApplePlatformMisc::CachedNetworkPath = InPath;
				nw_retain(UE::FApplePlatformMisc::CachedNetworkPath);

				ENetworkConnectionType ConnectionType = UE::FApplePlatformMisc::CheckNetworkConnectionType();

				if (FTaskGraphInterface::IsRunning())
				{
					if (ConnectionType != UE::FApplePlatformMisc::LastReportedNetworkType)
					{
						UE::FApplePlatformMisc::LastReportedNetworkType = ConnectionType;
						FFunctionGraphTask::CreateAndDispatchWhenReady([ConnectionType]() {
							FCoreDelegates::OnNetworkConnectionChanged.Broadcast(ConnectionType);
							FPlatformMisc::SetNetworkConnectionStatus(FPlatformMisc::NetworkConnectionTypeToStatus(ConnectionType));
						}, TStatId(), nullptr, ENamedThreads::GameThread);
					}

					// Adding a reference, so path survives to the task body.
					nw_retain(InPath);
					FFunctionGraphTask::CreateAndDispatchWhenReady(
					// nw_path_t is actually a pointer, so captured by value.
					[InPath] ()
					{
						UE::FApplePlatformMisc::NotifyAboutNetworkConnectionCharacteristicsChange(InPath);
					}, TStatId());
				}
				else
				{
					// Task graph not running yet (early startup). Set the initial status synchronously so it isn't silently dropped.
					UE::FApplePlatformMisc::LastReportedNetworkType = ConnectionType;
					FPlatformMisc::SetNetworkConnectionStatus(FPlatformMisc::NetworkConnectionTypeToStatus(ConnectionType));
				}
			}
		);

		nw_path_monitor_set_queue(UE::FApplePlatformMisc::PathMonitor, dispatch_get_main_queue());
		nw_path_monitor_start(UE::FApplePlatformMisc::PathMonitor);

		// The nw_path_monitor callback will fire shortly after start.
		// If the task graph is not yet running at that point, the callback sets the initial network status synchronously
	}
}

void FApplePlatformMisc::PlatformTearDown()
{
	if (UE::FApplePlatformMisc::PathMonitor != nullptr)
	{
		nw_path_monitor_cancel(UE::FApplePlatformMisc::PathMonitor);
		nw_release(UE::FApplePlatformMisc::PathMonitor);
		UE::FApplePlatformMisc::PathMonitor = nullptr;
	}

	if (UE::FApplePlatformMisc::CachedNetworkPath != nullptr)
	{
		nw_release(UE::FApplePlatformMisc::CachedNetworkPath);
		UE::FApplePlatformMisc::CachedNetworkPath = nullptr;
	}
}
