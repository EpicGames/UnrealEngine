// Copyright Epic Games, Inc. All Rights Reserved.

#include "Apple/ApplePlatformProcess.h"

#include "Apple/AppleStringUtils.h"
#include "Apple/ScopeAutoreleasePool.h"

#include "HAL/PlatformMisc.h"

const TCHAR* FApplePlatformProcess::UserDir()
{
	static TCHAR Result[FPlatformMisc::GetMaxPathLength()] = TEXT("");
	if (!Result[0])
	{
		SCOPED_AUTORELEASE_POOL;
		NSString *DocumentsFolder = [NSSearchPathForDirectoriesInDomains(NSDocumentDirectory, NSUserDomainMask, YES) objectAtIndex: 0];
		FPlatformString::CFStringToTCHAR((CFStringRef)DocumentsFolder, Result);
		FCString::Strcat(Result, TEXT("/"));
	}
	return Result;
}

const TCHAR* FApplePlatformProcess::UserTempDir()
{
	static FString TempDir;
	if (!TempDir.Len())
	{
		// The reason we use TMPDIR here is to support applications being detoured/remotely distributed (ShaderCompileWorker)
#if IS_PROGRAM
		TempDir = getenv("TMPDIR");
#else
		TempDir = NSTemporaryDirectory(); // NSTemporaryDirectory() can return a different temp directory than TMPDIR and detouring is not supported.
#endif
	}
	return *TempDir;
}

const TCHAR* FApplePlatformProcess::UserSettingsDir()
{
	return ApplicationSettingsDir();
}

const TCHAR* FApplePlatformProcess::UserHomeDir()
{
	static TCHAR Result[FPlatformMisc::GetMaxPathLength()] = TEXT("");
	if (!Result[0])
	{
		SCOPED_AUTORELEASE_POOL;
		FPlatformString::CFStringToTCHAR((CFStringRef)NSHomeDirectory(), Result);
	}
	return Result;
}

const TCHAR* FApplePlatformProcess::ApplicationSettingsDir()
{
	static TCHAR Result[FPlatformMisc::GetMaxPathLength()] = TEXT("");
	if (!Result[0])
	{
		SCOPED_AUTORELEASE_POOL;
		NSString *ApplicationSupportFolder = [NSSearchPathForDirectoriesInDomains(NSApplicationSupportDirectory, NSUserDomainMask, YES) objectAtIndex: 0];
		FPlatformString::CFStringToTCHAR((CFStringRef)ApplicationSupportFolder, Result);
		// @todo rocket this folder should be based on your company name, not just be hard coded to /Epic/
		FCString::Strcat(Result, TEXT("/Epic/"));
	}
	return Result;
}

FString FApplePlatformProcess::GetApplicationSettingsDir(const ApplicationSettingsContext& Settings)
{
	TCHAR Result[FPlatformMisc::GetMaxPathLength()] = TEXT("");
	SCOPED_AUTORELEASE_POOL;
	NSString* ApplicationSupportFolder = [NSSearchPathForDirectoriesInDomains(NSApplicationSupportDirectory, NSUserDomainMask, YES) objectAtIndex:0];
	FPlatformString::CFStringToTCHAR((CFStringRef) ApplicationSupportFolder, Result);
	if (Settings.bIsEpic)
	{
		// @todo rocket this folder should be based on your company name, not just be hard coded to /Epic/
		FCString::Strcat(Result, TEXT("/Epic/"));
	}
	else
	{
		FCString::Strcat(Result, TEXT("/"));
	}
	return FString(Result);
}

