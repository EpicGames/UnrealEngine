// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ApplePlatformStackWalk.mm: Apple implementations of stack walk functions
=============================================================================*/

#include "Apple/ApplePlatformStackWalk.h"
#include "Apple/ApplePlatformSymbolication.h"
#include "Containers/StringConv.h"
#include "Apple/PreAppleSystemHeaders.h"
#include <execinfo.h>
#include <dlfcn.h>
#include <mach-o/dyld.h>
#include <cxxabi.h>
#include <signal.h>
#include "Apple/PostAppleSystemHeaders.h"
#include "CoreGlobals.h"
#include "Misc/ScopeLock.h"
#include "Templates/UnrealTemplate.h"
#include "HAL/IConsoleManager.h"

#if PLATFORM_MAC
#include "Mac/MacSystemIncludes.h"
#include "PLCrashReporter.h"
#endif

#if PLATFORM_MAC
static TAutoConsoleVariable<int32> CVarApplePlatformThreadCallStackEnabled(
	TEXT("ApplePlatformThreadStackWalk.Enable"),
	1,
	TEXT("If > 0, then when requesting callstack info about a thread will raise a signal and gather that information."));
static TAutoConsoleVariable<float> CVarApplePlatformThreadCallStackMaxWait(
	TEXT("ApplePlatformThreadStackWalk.MaxWait"),
	60.0f,
	TEXT("The number of seconds allowed to spin before killing the process, with the assumption the signal handler has hung."));
#endif

// Set while a stack walk is in progress on the current thread. Without this guard,
// a failure inside the stack walker can recurse until the stack is exhausted.
static thread_local bool GStackWalkInProgress = false;

// Returns true if it is safe to start a stack walk on the current thread. Returns false
// if one is already in progress (the caller should bail out); in that case, an optional
// caller-supplied buffer is populated with a short marker so its output path is not
// left with stale data.
static bool CanProcessStackWalk(ANSICHAR* HumanReadableString = nullptr, SIZE_T HumanReadableStringSize = 0)
{
	if (GStackWalkInProgress)
	{
		if (HumanReadableString && HumanReadableStringSize > 0)
		{
			FCStringAnsi::Strncpy(HumanReadableString, "[stack walker re-entered; skipping]\n", HumanReadableStringSize);
		}
		return false;
	}
	return true;
}

// Internal helper functions not exposed to public
int32 GetModuleImageSize( const struct mach_header* Header )
{
	struct load_command *CurrentCommand = (struct load_command *)( (char *)Header + sizeof(struct mach_header_64) );
	int32 ModuleSize = 0;
	
	// Check header
	if( Header->magic != MH_MAGIC_64 )
		return 0;
	
	for( int32 i = 0; i < Header->ncmds; i++ )
	{
		if( CurrentCommand->cmd == LC_SEGMENT )
		{
			struct segment_command *SegmentCommand = (struct segment_command *) CurrentCommand;
			ModuleSize += SegmentCommand->vmsize;
		}
		else if( CurrentCommand->cmd == LC_SEGMENT_64 )
		{
			struct segment_command_64 *SegmentCommand = (struct segment_command_64 *) CurrentCommand;
			ModuleSize += SegmentCommand->vmsize;
		}

		CurrentCommand = (struct load_command *)( (char *)CurrentCommand + CurrentCommand->cmdsize );
	}
	
	return ModuleSize;
}

int32 GetModuleTimeStamp( const struct mach_header* Header )
{
	struct load_command *CurrentCommand = (struct load_command *)( (char *)Header + sizeof(struct mach_header_64) );
	int32 ModuleSize = 0;
	
	// Check header
	if( Header->magic != MH_MAGIC_64 )
		return 0;
	
	for( int32 i = 0; i < Header->ncmds; i++ )
	{
		if( CurrentCommand->cmd == LC_LOAD_DYLIB ) //LC_ID_DYLIB )
		{
			struct dylib_command *DylibCommand = (struct dylib_command *) CurrentCommand;
			return DylibCommand->dylib.timestamp;
		}
		
		CurrentCommand = (struct load_command *)( (char *)CurrentCommand + CurrentCommand->cmdsize );
	}
	
	return 0;
}

static void AsyncSafeProgramCounterToSymbolInfo( uint64 ProgramCounter, FProgramCounterSymbolInfo& out_SymbolInfo )
{
	// Store the PC into the info structure
	out_SymbolInfo.ProgramCounter = ProgramCounter;
	
	Dl_info DylibInfo;
	int32 Result = dladdr((const void*)ProgramCounter, &DylibInfo);
	if (Result == 0)
	{
		return;
	}

#if PLATFORM_MAC && IS_PROGRAM // On the Mac the crash report client can resymbolise
	if (DylibInfo.dli_sname)
	{
		// Mangled function name
		FCStringAnsi::Sprintf(out_SymbolInfo.FunctionName, "%s ", DylibInfo.dli_sname);
	}
	else
	{
		// Unknown!
		FCStringAnsi::Sprintf(out_SymbolInfo.FunctionName, "[Unknown]() ");
	}
#else // But on iOS the best we can do is demangle
	int32 Status = 0;
	ANSICHAR* DemangledName = nullptr;
	
	// Increased the size of the demangle destination to reduce the chances that abi::__cxa_demangle will allocate
	// this causes the app to hang as malloc isn't signal handler safe. Ideally we wouldn't call this function in a handler.
	size_t DemangledNameLen = 65536;
	static ANSICHAR DemangledNameBuffer[65536]= {0};
	DemangledName = abi::__cxa_demangle(DylibInfo.dli_sname, DemangledNameBuffer, &DemangledNameLen, &Status);
	
	if (DemangledName)
	{
		// C++ function
		FCStringAnsi::Sprintf(out_SymbolInfo.FunctionName, "%s ", DemangledName);
	}
	else if (DylibInfo.dli_sname && strchr(DylibInfo.dli_sname, ']'))
	{
		// ObjC function
		FCStringAnsi::Sprintf(out_SymbolInfo.FunctionName, "%s ", DylibInfo.dli_sname);
	}
	else if(DylibInfo.dli_sname)
	{
		// C function
		FCStringAnsi::Sprintf(out_SymbolInfo.FunctionName, "%s() ", DylibInfo.dli_sname);
	}
	else
	{
		// Unknown!
		FCStringAnsi::Sprintf(out_SymbolInfo.FunctionName, "[Unknown]() ");
	}
#endif
	
	// No line number found.
	FCStringAnsi::Strcat(out_SymbolInfo.Filename, "Unknown");
	out_SymbolInfo.LineNumber = 0;
	
	// Write out Module information.
	ANSICHAR* DylibPath = (ANSICHAR*)DylibInfo.dli_fname;
	ANSICHAR* DylibName = FCStringAnsi::Strrchr(DylibPath, '/');
	if (DylibName)
	{
		DylibName += 1;
	}
	else
	{
		DylibName = DylibPath;
	}
	FCStringAnsi::Strcpy(out_SymbolInfo.ModuleName, DylibName);
}

uint32 FApplePlatformStackWalk::CaptureStackBackTrace( uint64* BackTrace, uint32 MaxDepth, void* Context )
{
	// Make sure we have place to store the information before we go through the process of raising
	// an exception and handling it.
	if( BackTrace == nullptr || MaxDepth == 0 )
	{
		return 0;
	}

#if PLATFORM_MAC && !defined(USE_UNTESTED_PL_CRASHREPORTER)
	if(Context)
	{
		return plcrashreporter_backtrace((void**)BackTrace, MaxDepth);
	}
#endif
	return backtrace((void**)BackTrace, MaxDepth);
}

bool FApplePlatformStackWalk::ProgramCounterToHumanReadableString( int32 CurrentCallDepth, uint64 ProgramCounter, ANSICHAR* HumanReadableString, SIZE_T HumanReadableStringSize, FGenericCrashContext* Context )
{
	
	//
	// Callstack lines should be written in this standard format
	//
	//	0xaddress module!func [file]
	// 
	// E.g. 0x045C8D01 OrionClient.self!UEngine::PerformError() [D:\Epic\Orion\Engine\Source\Runtime\Engine\Private\UnrealEngine.cpp:6481]
	//
	// Module may be omitted, everything else should be present, or substituted with a string that conforms to the expected type
	//
	// E.g 0x00000000 UnknownFunction []
	//
	// 

	Dl_info DylibInfo;
	int32 Result = dladdr((const void*)ProgramCounter, &DylibInfo);
	if (Result == 0)
	{
		return false;
	}

	FProgramCounterSymbolInfo SymbolInfo;
	if(!Context)
	{
		ProgramCounterToSymbolInfo( ProgramCounter, SymbolInfo );
	}
	else
	{
		AsyncSafeProgramCounterToSymbolInfo(ProgramCounter, SymbolInfo);
	}

	ANSICHAR TempArray[MAX_SPRINTF];

	// Write out prefix, address, module, and function na,e
	FCStringAnsi::Sprintf(TempArray, "0x%08x %s!%s ", (uint32)ProgramCounter, SymbolInfo.ModuleName, SymbolInfo.FunctionName);
	
	FCStringAnsi::StrncatTruncateDest(HumanReadableString, HumanReadableStringSize, TempArray);

	// Get filename.
	{
		ANSICHAR FileNameLine[MAX_SPRINTF];
		
		if(SymbolInfo.LineNumber == 0)
		{
			// No line number. Print out the logical address instead.
			FCStringAnsi::Sprintf(FileNameLine, " [UnknownFile]) ", ProgramCounter);
		}
		else
		{
			// try to add source file and line number, too
			FCStringAnsi::Sprintf(FileNameLine, " [%s:%d] ", SymbolInfo.Filename, SymbolInfo.LineNumber);
		}
		
		FCStringAnsi::StrncatTruncateDest(HumanReadableString, HumanReadableStringSize, FileNameLine);
	}

	// For the crash reporting code this needs a Windows line ending, the caller is responsible for the '\n'
	FCStringAnsi::StrncatTruncateDest(HumanReadableString, HumanReadableStringSize, "\r");

	return true;
}

void FApplePlatformStackWalk::ProgramCounterToSymbolInfo( uint64 ProgramCounter, FProgramCounterSymbolInfo& out_SymbolInfo )
{
	bool bOK = FApplePlatformSymbolication::SymbolInfoForAddress(ProgramCounter, out_SymbolInfo);
	if(!bOK)
	{
		AsyncSafeProgramCounterToSymbolInfo(ProgramCounter, out_SymbolInfo);
	}
}

int32 FApplePlatformStackWalk::GetProcessModuleCount()
{
	return (int32)_dyld_image_count();
}

int32 FApplePlatformStackWalk::GetProcessModuleSignatures(FStackWalkModuleInfo *ModuleSignatures, const int32 ModuleSignaturesSize)
{
	int32 ModuleCount = GetProcessModuleCount();

	TArray<FStackWalkModuleInfo> TempModules;
	TempModules.Reserve(ModuleCount);

	// The shared cache breaks our module size calculations. When we sum segment vmsizes from the mach_header,
	// we get inflated sizes because the cache splits segments apart - they're not contiguous. 
	// This causes wrong module attribution when checking address ranges.

	// So we use dladdr() to make it work as _dyld_get_image_header() gives us the header pointer, but
	// for cached frameworks that's not where the code actually lives. dladdr's dli_fbase gives
	// us the real base address and what we need.
	// However the order in which we get these aren't sorted by base address.
	// So we've got to collect them all.
	// Then we calculate size as the distance to the next module instead of summing segments.
	for (int32 ModuleIndex = 0; ModuleIndex < ModuleCount; ModuleIndex++)
	{
		const struct mach_header* Header = _dyld_get_image_header(ModuleIndex);
		const ANSICHAR* ImageName = _dyld_get_image_name(ModuleIndex);

		if (Header && ImageName)
		{
			FStackWalkModuleInfo Info;
			Dl_info DylibInfo;
			if (dladdr((const void*)Header, &DylibInfo) && DylibInfo.dli_fbase)
			{
				Info.BaseOfImage = (uint64)DylibInfo.dli_fbase;
			}
			else
			{
				// Fallback to the busted way in case things go wrong
				// so at least we have something
				Info.BaseOfImage = (uint64)Header;
			}

			Info.ImageSize = 0;

			FCString::Strcpy(Info.ImageName, ANSI_TO_TCHAR(ImageName));
			FCString::Strcpy(Info.LoadedImageName, ANSI_TO_TCHAR(ImageName));
			FCString::Strcpy(Info.ModuleName, ANSI_TO_TCHAR(ImageName));
			Info.PdbAge = 0;
			Info.PdbSig = 0;
			FMemory::Memzero(&Info.PdbSig70, sizeof(Info.PdbSig70));
			Info.TimeDateStamp = GetModuleTimeStamp(Header);

			TempModules.Add(Info);
		}
	}
	
	TempModules.Sort([](const FStackWalkModuleInfo& A, const FStackWalkModuleInfo& B)
	{
		return A.BaseOfImage < B.BaseOfImage;
	});

	for (int32 Idx = 0; Idx < TempModules.Num(); Idx++)
	{
		if (Idx + 1 < TempModules.Num())
		{
			uint64 CurrentBase = TempModules[Idx].BaseOfImage;
			uint64 NextBase = TempModules[Idx + 1].BaseOfImage;

			TempModules[Idx].ImageSize = (uint32)(NextBase - CurrentBase);
		}
		else
		{
			TempModules[Idx].ImageSize = 0xFFFFFFFF;
		}
	}

	int32 SignatureCount = FMath::Min(TempModules.Num(), ModuleSignaturesSize);
	for (int32 Idx = 0; Idx < SignatureCount; Idx++)
	{
		ModuleSignatures[Idx] = TempModules[Idx];
	}

	return SignatureCount;
}

// Consider this for iOS
#if PLATFORM_MAC
FCriticalSection GThreadCallStackMutex;

// These are used in the SIGUSR2 callback to hand over the user provided buffer to copy to
ANSICHAR* GThreadCallStack = nullptr;

// The user provided buffer for CaptureThreadStackBackTrace to get just the backtrace of program counters
uint64* GThreadBackTrace = nullptr;

// The user provided size for either the Callstack buffer or the BackTrace buffer
SIZE_T GThreadCallStackSize = 0;

// The number of program counters added to the backtrace
uint32 GThreadBackTraceCount = 0;

// Used to determine if a SIGUSR2 signal hander is currently running
bool GThreadCallStackInUse = false;

static void RaiseSIGUSR2ForThreadAndWait(uint32 ThreadId)
{
	if (pthread_kill(pthread_from_mach_thread_np(ThreadId), SIGUSR2) == 0)
	{
		float MaxWaitTime   = CVarApplePlatformThreadCallStackMaxWait.AsVariable()->GetFloat();
		float TotalWaitTime = 0.0f;

		// Spin until the SignalHandler is done and has set GThreadCallStackInUse to false
		while (GThreadCallStackInUse)
		{
			if (TotalWaitTime > MaxWaitTime)
			{
				// We have gone over our wait time and now should crash versus waiting possibly forever
				*(int*)0x10 = 0x11;
			}

			// sleep a small amount until MaxWaitTime.
			// Depending on the use of this function this value my need to be adjusted or messed with
			usleep(1000);
			TotalWaitTime += 0.001f;
		}
	}
}
#endif

void FApplePlatformStackWalk::ThreadStackWalkAndDump(ANSICHAR* HumanReadableString, SIZE_T HumanReadableStringSize, int32 IgnoreCount, uint32 ThreadId)
{
	if (!CanProcessStackWalk(HumanReadableString, HumanReadableStringSize))
	{
		return;
	}
	TGuardValue<bool> ReentryGuard(GStackWalkInProgress, true);

// Consider this for iOS
#if PLATFORM_MAC
	if (CVarApplePlatformThreadCallStackEnabled.AsVariable()->GetInt())
	{
		FScopeLock Lock(&GThreadCallStackMutex);

		GThreadCallStack      = HumanReadableString;
		GThreadCallStackSize  = HumanReadableStringSize;
		GThreadCallStackInUse = true;

		RaiseSIGUSR2ForThreadAndWait(ThreadId);

		GThreadCallStack     = nullptr;
		GThreadCallStackSize = 0;
	}
#endif
}

uint32 FApplePlatformStackWalk::CaptureThreadStackBackTrace(uint64 ThreadId, uint64* BackTrace, uint32 MaxDepth, void* Context)
{
// Consider this for iOS
#if PLATFORM_MAC
	GThreadBackTraceCount = 0;

	if (CVarApplePlatformThreadCallStackEnabled.AsVariable()->GetInt())
	{
		FScopeLock Lock(&GThreadCallStackMutex);

		GThreadBackTrace      = BackTrace;
		GThreadCallStackSize  = MaxDepth;
		GThreadCallStackInUse = true;

		RaiseSIGUSR2ForThreadAndWait(ThreadId);

		GThreadBackTrace     = nullptr;
		GThreadCallStackSize = 0;
	}

	return GThreadBackTraceCount;
#else
	return 0;
#endif
}

void CreateExceptionInfoString(int32 Signal, struct __siginfo* Info)
{
	FString ErrorString = "Unhandled Exception: ";
	
#define HANDLE_CASE(a,b) case a: ErrorString += TEXT(#a #b); break;
	
	switch (Signal)
	{
		case SIGSEGV:
			ErrorString += TEXT("SIGSEGV segmentation violation at address ");
			ErrorString += FString::Printf(TEXT("0x%016" UINT64_x_FMT), (uint64)(uint32*)Info->si_addr);
			break;
		case SIGBUS:
			ErrorString += TEXT("SIGBUS bus error at address ");
			ErrorString += FString::Printf(TEXT("0x%016" UINT64_x_FMT), (uint64)(uint32*)Info->si_addr);
			break;
			
		HANDLE_CASE(SIGINT, "interrupt program")
		HANDLE_CASE(SIGQUIT, "quit program")
		HANDLE_CASE(SIGILL, "illegal instruction")
		HANDLE_CASE(SIGTRAP, "trace trap")
		HANDLE_CASE(SIGABRT, "abort() call")
		HANDLE_CASE(SIGFPE, "floating-point exception")
		HANDLE_CASE(SIGKILL, "kill program")
		HANDLE_CASE(SIGSYS, "non-existent system call invoked")
		HANDLE_CASE(SIGPIPE, "write on a pipe with no reader")
		HANDLE_CASE(SIGTERM, "software termination signal")
		HANDLE_CASE(SIGSTOP, "stop")

		default:
			ErrorString += FString::Printf(TEXT("0x%08x"), (uint32)Signal);
	}
	
#if WITH_EDITORONLY_DATA
	FCString::Strncpy(GErrorExceptionDescription, *ErrorString, FMath::Min(ErrorString.Len() + 1, (int32)UE_ARRAY_COUNT(GErrorExceptionDescription)));
#endif
#undef HANDLE_CASE
}

int32 ReportCrash(ucontext_t *Context, int32 Signal, struct __siginfo* Info)
{
	static bool GAlreadyCreatedMinidump = false;
	
	// Only create a minidump the first time this function is called.
	// (Can be called the first time from the RenderThread, then a second time from the MainThread.)
	if ( GAlreadyCreatedMinidump == false )
	{
		GAlreadyCreatedMinidump = true;

		// No malloc in a signal handler as it is unsafe & will deadlock the application!
		const SIZE_T StackTraceSize = 65535;
		static ANSICHAR StackTrace[ StackTraceSize ];
		StackTrace[0] = 0;
		
		// Walk the stack and dump it to the allocated memory.
		FPlatformStackWalk::StackWalkAndDump( StackTrace, StackTraceSize, Info->si_addr, Context );
#if WITH_EDITORONLY_DATA
		FCString::StrncatTruncateDest( GErrorHist, UE_ARRAY_COUNT(GErrorHist), ANSI_TO_TCHAR(StackTrace) );
		CreateExceptionInfoString(Signal, Info);
#endif
	}

	return 0;
}
