// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ApplePlatformString.mm: Mac implementations of string functions
=============================================================================*/

#include "Apple/ApplePlatformString.h"
#include "Containers/UnrealString.h"
#include "Containers/StringConv.h"

#if PLATFORM_MAC
#include "Mac/MacSystemIncludes.h"
#elif PLATFORM_IOS
#include "IOS/IOSSystemIncludes.h"
#endif

void FApplePlatformString::CFStringToTCHAR( CFStringRef CFStr, TCHAR *TChar )
{
	const SIZE_T Length = CFStringGetLength( CFStr );
	CFRange Range = CFRangeMake( 0, Length );
	CFStringGetBytes( CFStr, Range, sizeof( TCHAR ) == 4 ? kCFStringEncodingUTF32LE : kCFStringEncodingUnicode, '?', false, ( uint8 *)TChar, Length * sizeof( TCHAR ) + 1, NULL );
	TChar[Length] = 0;
}

CFStringRef FApplePlatformString::TCHARToCFString( const TCHAR *TChar )
{
	const SIZE_T Length = Strlen( TChar );
	CFStringRef String = CFStringCreateWithBytes( kCFAllocatorDefault, ( const uint8 *)TChar, Length * sizeof( TCHAR ), sizeof( TCHAR ) == 4 ? kCFStringEncodingUTF32LE : kCFStringEncodingUnicode, false );
	// we are getting some crashes on strings that aren't converting - so, instead of crashing, print them out instead
	ensureMsgf(String, TEXT("Failed to allocated CFString for '%s' -- Length id %zu"), TChar ? TChar : TEXT("nullptr"), Length);
	return String;
}

