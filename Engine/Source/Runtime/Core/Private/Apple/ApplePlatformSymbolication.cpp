// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ApplePlatformSymbolication.cpp: Apple platform implementation of symbolication
=============================================================================*/

#include "Apple/ApplePlatformSymbolication.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformProcess.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CoreStats.h"
#include "Containers/Map.h"
#include "CoreGlobals.h"
#include <CoreFoundation/CoreFoundation.h>
#include <mach/mach.h>

#include "Apple/AppleStringUtils.h"

extern "C"
{
	struct CSRange
	{
		uint64 Location;
		uint64 Length;
	};
	
	typedef FApplePlatformSymbolCache CSTypeRef;
	typedef CSTypeRef CSSymbolicatorRef;
	typedef CSTypeRef CSSourceInfoRef;
	typedef CSTypeRef CSSymbolRef;
	typedef CSTypeRef CSSymbolOwnerRef;
	
	typedef int (^CSSymbolIterator)(CSSymbolRef Symbol);
	typedef int (^CSSourceInfoIterator)(CSSourceInfoRef SourceInfo);
	
	#define kCSNow								0x80000000u
	
	typedef Boolean (*CSEqualPtr)(CSTypeRef Cs1, CSTypeRef Cs2);
	typedef Boolean (*CSIsNullPtr)(CSTypeRef CS);
	typedef void (*CSReleasePtr)(CSTypeRef CS);
	typedef void (*CSRetainPtr)(CSTypeRef CS);
	
	typedef CSSymbolicatorRef (*CSSymbolicatorCreateWithPidPtr)(pid_t pid);
	typedef CSSymbolicatorRef (*CSSymbolicatorCreateWithPathAndArchitecturePtr)(const char* path, cpu_type_t type);
	
	typedef CSSymbolRef (*CSSymbolicatorGetSymbolWithAddressAtTimePtr)(CSSymbolicatorRef Symbolicator, vm_address_t Address, uint64_t Time);
	typedef CSSourceInfoRef (*CSSymbolicatorGetSourceInfoWithAddressAtTimePtr)(CSSymbolicatorRef Symbolicator, vm_address_t Address, uint64_t Time);
	typedef CSSymbolOwnerRef (*CSSymbolicatorGetSymbolOwnerWithUUIDAtTimePtr)(CSSymbolicatorRef Symbolicator, CFUUIDRef UUID, uint64_t Time);
	typedef CSSymbolOwnerRef (*CSSymbolicatorGetSymbolOwnerPtr)(CSSymbolicatorRef cs);
	typedef int (*CSSymbolicatorForeachSymbolAtTimePtr)(CSSymbolicatorRef Symbolicator, uint64_t Time, CSSymbolIterator It);
	
	typedef const char* (*CSSymbolGetNamePtr)(CSSymbolRef Symbol);
	typedef CSRange (*CSSymbolGetRangePtr)(CSSymbolRef Symbol);
	typedef CSSymbolOwnerRef (*CSSourceInfoGetSymbolOwnerPtr)(CSSourceInfoRef Info);
	typedef CSSymbolOwnerRef (*CSSymbolGetSymbolOwnerPtr)(CSSymbolRef Sym);
	typedef int (*CSSymbolForeachSourceInfoPtr)(CSSymbolRef Sym, CSSourceInfoIterator It);
	
	typedef const char* (*CSSymbolOwnerGetNamePtr)(CSSymbolOwnerRef Owner);
	typedef CFUUIDRef (*CSSymbolOwnerGetUUIDPtr)(CSSymbolOwnerRef Owner);
	typedef vm_address_t (*CSSymbolOwnerGetBaseAddressPtr)(CSSymbolOwnerRef Owner);
	
	typedef int (*CSSourceInfoGetLineNumberPtr)(CSSourceInfoRef Info);
	typedef const char* (*CSSourceInfoGetPathPtr)(CSSourceInfoRef Info);
	typedef CSRange (*CSSourceInfoGetRangePtr)(CSSourceInfoRef Info);
	typedef CSSymbolRef (*CSSourceInfoGetSymbolPtr)(CSSourceInfoRef Info);
}

static bool GAllowApplePlatformSymbolication = false;
static void* GCoreSymbolicationHandle = nullptr;
static CSEqualPtr CSEqual = nullptr;
static CSIsNullPtr CSIsNull = nullptr;
static CSReleasePtr CSRelease = nullptr;
static CSRetainPtr CSRetain = nullptr;
static CSSymbolicatorCreateWithPidPtr CSSymbolicatorCreateWithPid = nullptr;
static CSSymbolicatorCreateWithPathAndArchitecturePtr CSSymbolicatorCreateWithPathAndArchitecture = nullptr;
static CSSymbolicatorGetSymbolWithAddressAtTimePtr CSSymbolicatorGetSymbolWithAddressAtTime = nullptr;
static CSSymbolicatorGetSourceInfoWithAddressAtTimePtr CSSymbolicatorGetSourceInfoWithAddressAtTime = nullptr;
static CSSymbolicatorGetSymbolOwnerWithUUIDAtTimePtr CSSymbolicatorGetSymbolOwnerWithUUIDAtTime = nullptr;
static CSSymbolicatorGetSymbolOwnerPtr CSSymbolicatorGetSymbolOwner = nullptr;
static CSSymbolicatorForeachSymbolAtTimePtr CSSymbolicatorForeachSymbolAtTime = nullptr;
static CSSymbolGetNamePtr CSSymbolGetName = nullptr;
static CSSymbolGetRangePtr CSSymbolGetRange = nullptr;
static CSSourceInfoGetSymbolOwnerPtr CSSourceInfoGetSymbolOwner = nullptr;
static CSSymbolGetSymbolOwnerPtr CSSymbolGetSymbolOwner = nullptr;
static CSSymbolForeachSourceInfoPtr CSSymbolForeachSourceInfo = nullptr;
static CSSymbolOwnerGetNamePtr CSSymbolOwnerGetName = nullptr;
static CSSymbolOwnerGetUUIDPtr CSSymbolOwnerGetUUID = nullptr;
static CSSymbolOwnerGetBaseAddressPtr CSSymbolOwnerGetBaseAddress = nullptr;
static CSSourceInfoGetLineNumberPtr CSSourceInfoGetLineNumber = nullptr;
static CSSourceInfoGetPathPtr CSSourceInfoGetPath = nullptr;
static CSSourceInfoGetRangePtr CSSourceInfoGetRange = nullptr;
static CSSourceInfoGetSymbolPtr CSSourceInfoGetSymbol = nullptr;

FApplePlatformSymbolDatabase::FApplePlatformSymbolDatabase()
{
    GenericDB = MakeShareable( new FGenericPlatformSymbolDatabase() );
	AppleDB.Buffer0 = nullptr;
	AppleDB.Buffer1 = nullptr;
}

FApplePlatformSymbolDatabase::FApplePlatformSymbolDatabase(FApplePlatformSymbolDatabase const& Other)
{
	operator=(Other);
}

FApplePlatformSymbolDatabase::~FApplePlatformSymbolDatabase()
{
	if(GCoreSymbolicationHandle && !CSIsNull(AppleDB))
	{
		CSRelease(AppleDB);
	}
	AppleDB.Buffer0 = nullptr;
	AppleDB.Buffer1 = nullptr;
    GenericDB = nullptr;
}

FApplePlatformSymbolDatabase& FApplePlatformSymbolDatabase::operator=(FApplePlatformSymbolDatabase const& Other)
{
	if(this != &Other)
	{
		GenericDB = Other.GenericDB;
		AppleDB = Other.AppleDB;
        if (AppleDB.Buffer0 != nullptr && AppleDB.Buffer1 != nullptr)
        {
            CSRetain(AppleDB);
        }
	}
	return *this;
}

void FApplePlatformSymbolication::EnableCoreSymbolication(bool const bEnable)
{
	GAllowApplePlatformSymbolication = bEnable;
	if(bEnable && !GCoreSymbolicationHandle)
	{
		GCoreSymbolicationHandle = FPlatformProcess::GetDllHandle(TEXT("/System/Library/PrivateFrameworks/CoreSymbolication.framework/Versions/Current/CoreSymbolication"));
		if(GCoreSymbolicationHandle)
		{
			CSEqual = (CSEqualPtr)FPlatformProcess::GetDllExport(GCoreSymbolicationHandle, TEXT("CSEqual"));
			GAllowApplePlatformSymbolication &= (CSEqual != nullptr);
			
			CSIsNull = (CSIsNullPtr)FPlatformProcess::GetDllExport(GCoreSymbolicationHandle, TEXT("CSIsNull"));
			GAllowApplePlatformSymbolication &= (CSIsNull != nullptr);
			
			CSRelease = (CSReleasePtr)FPlatformProcess::GetDllExport(GCoreSymbolicationHandle, TEXT("CSRelease"));
			GAllowApplePlatformSymbolication &= (CSRelease != nullptr);
			
			CSRetain = (CSRetainPtr)FPlatformProcess::GetDllExport(GCoreSymbolicationHandle, TEXT("CSRetain"));
			GAllowApplePlatformSymbolication &= (CSRetain != nullptr);
			
			CSSymbolicatorCreateWithPid = (CSSymbolicatorCreateWithPidPtr)FPlatformProcess::GetDllExport(GCoreSymbolicationHandle, TEXT("CSSymbolicatorCreateWithPid"));
			GAllowApplePlatformSymbolication &= (CSSymbolicatorCreateWithPid != nullptr);
			
			CSSymbolicatorCreateWithPathAndArchitecture = (CSSymbolicatorCreateWithPathAndArchitecturePtr)FPlatformProcess::GetDllExport(GCoreSymbolicationHandle, TEXT("CSSymbolicatorCreateWithPathAndArchitecture"));
			GAllowApplePlatformSymbolication &= (CSSymbolicatorCreateWithPathAndArchitecture != nullptr);
			
			CSSymbolicatorGetSymbolWithAddressAtTime = (CSSymbolicatorGetSymbolWithAddressAtTimePtr)FPlatformProcess::GetDllExport(GCoreSymbolicationHandle, TEXT("CSSymbolicatorGetSymbolWithAddressAtTime"));
			GAllowApplePlatformSymbolication &= (CSSymbolicatorGetSymbolWithAddressAtTime != nullptr);
			
			CSSymbolicatorGetSourceInfoWithAddressAtTime = (CSSymbolicatorGetSourceInfoWithAddressAtTimePtr)FPlatformProcess::GetDllExport(GCoreSymbolicationHandle, TEXT("CSSymbolicatorGetSourceInfoWithAddressAtTime"));
			GAllowApplePlatformSymbolication &= (CSSymbolicatorGetSourceInfoWithAddressAtTime != nullptr);
			
			CSSymbolicatorGetSymbolOwnerWithUUIDAtTime = (CSSymbolicatorGetSymbolOwnerWithUUIDAtTimePtr)FPlatformProcess::GetDllExport(GCoreSymbolicationHandle, TEXT("CSSymbolicatorGetSymbolOwnerWithUUIDAtTime"));
			GAllowApplePlatformSymbolication &= (CSSymbolicatorGetSymbolOwnerWithUUIDAtTime != nullptr);
			
			CSSymbolicatorGetSymbolOwner = (CSSymbolicatorGetSymbolOwnerPtr)FPlatformProcess::GetDllExport(GCoreSymbolicationHandle, TEXT("CSSymbolicatorGetSymbolOwner"));
			GAllowApplePlatformSymbolication &= (CSSymbolicatorGetSymbolOwner != nullptr);
			
			CSSymbolicatorForeachSymbolAtTime = (CSSymbolicatorForeachSymbolAtTimePtr)FPlatformProcess::GetDllExport(GCoreSymbolicationHandle, TEXT("CSSymbolicatorForeachSymbolAtTime"));
			GAllowApplePlatformSymbolication &= (CSSymbolicatorForeachSymbolAtTime != nullptr);
			
			CSSymbolGetName = (CSSymbolGetNamePtr)FPlatformProcess::GetDllExport(GCoreSymbolicationHandle, TEXT("CSSymbolGetName"));
			GAllowApplePlatformSymbolication &= (CSSymbolGetName != nullptr);
			
			CSSymbolGetRange = (CSSymbolGetRangePtr)FPlatformProcess::GetDllExport(GCoreSymbolicationHandle, TEXT("CSSymbolGetRange"));
			GAllowApplePlatformSymbolication &= (CSSymbolGetRange != nullptr);
			
			CSSourceInfoGetSymbolOwner = (CSSourceInfoGetSymbolOwnerPtr)FPlatformProcess::GetDllExport(GCoreSymbolicationHandle, TEXT("CSSourceInfoGetSymbolOwner"));
			GAllowApplePlatformSymbolication &= (CSSourceInfoGetSymbolOwner != nullptr);
			
			CSSymbolGetSymbolOwner = (CSSymbolGetSymbolOwnerPtr)FPlatformProcess::GetDllExport(GCoreSymbolicationHandle, TEXT("CSSymbolGetSymbolOwner"));
			GAllowApplePlatformSymbolication &= (CSSymbolGetSymbolOwner != nullptr);
			
			CSSymbolOwnerGetBaseAddress = (CSSymbolOwnerGetBaseAddressPtr)FPlatformProcess::GetDllExport(GCoreSymbolicationHandle, TEXT("CSSymbolOwnerGetBaseAddress"));
			GAllowApplePlatformSymbolication &= (CSSymbolOwnerGetBaseAddress != nullptr);
			
			CSSymbolForeachSourceInfo = (CSSymbolForeachSourceInfoPtr)FPlatformProcess::GetDllExport(GCoreSymbolicationHandle, TEXT("CSSymbolForeachSourceInfo"));
			GAllowApplePlatformSymbolication &= (CSSymbolForeachSourceInfo != nullptr);
			
			CSSymbolOwnerGetName = (CSSymbolOwnerGetNamePtr)FPlatformProcess::GetDllExport(GCoreSymbolicationHandle, TEXT("CSSymbolOwnerGetName"));
			GAllowApplePlatformSymbolication &= (CSSymbolOwnerGetName != nullptr);
			
			CSSymbolOwnerGetUUID = (CSSymbolOwnerGetUUIDPtr)FPlatformProcess::GetDllExport(GCoreSymbolicationHandle, TEXT("CSSymbolOwnerGetUUID"));
			GAllowApplePlatformSymbolication &= (CSSymbolOwnerGetUUID != nullptr);
			
			CSSourceInfoGetLineNumber = (CSSourceInfoGetLineNumberPtr)FPlatformProcess::GetDllExport(GCoreSymbolicationHandle, TEXT("CSSourceInfoGetLineNumber"));
			GAllowApplePlatformSymbolication &= (CSSourceInfoGetLineNumber != nullptr);
			
			CSSourceInfoGetPath = (CSSourceInfoGetPathPtr)FPlatformProcess::GetDllExport(GCoreSymbolicationHandle, TEXT("CSSourceInfoGetPath"));
			GAllowApplePlatformSymbolication &= (CSSourceInfoGetPath != nullptr);
			
			CSSourceInfoGetRange = (CSSourceInfoGetRangePtr)FPlatformProcess::GetDllExport(GCoreSymbolicationHandle, TEXT("CSSourceInfoGetRange"));
			GAllowApplePlatformSymbolication &= (CSSourceInfoGetRange != nullptr);
			
			CSSourceInfoGetSymbol = (CSSourceInfoGetSymbolPtr)FPlatformProcess::GetDllExport(GCoreSymbolicationHandle, TEXT("CSSourceInfoGetSymbol"));
			GAllowApplePlatformSymbolication &= (CSSourceInfoGetSymbol != nullptr);
		}
		else
		{
			GAllowApplePlatformSymbolication = false;
		}
	}
	else if(!bEnable && GCoreSymbolicationHandle)
	{
		CSIsNull = nullptr;
		CSRelease = nullptr;
		CSRetain = nullptr;
		CSSymbolicatorCreateWithPid = nullptr;
		CSSymbolicatorCreateWithPathAndArchitecture = nullptr;
		CSSymbolicatorGetSymbolWithAddressAtTime = nullptr;
		CSSymbolicatorGetSourceInfoWithAddressAtTime = nullptr;
		CSSymbolicatorGetSymbolOwnerWithUUIDAtTime = nullptr;
		CSSymbolicatorGetSymbolOwner = nullptr;
		CSSymbolicatorForeachSymbolAtTime = nullptr;
		CSSymbolGetName = nullptr;
		CSSymbolGetRange = nullptr;
		CSSourceInfoGetSymbolOwner = nullptr;
		CSSymbolGetSymbolOwner = nullptr;
		CSSymbolForeachSourceInfo = nullptr;
		CSSymbolOwnerGetName = nullptr;
		CSSymbolOwnerGetUUID = nullptr;
		CSSymbolOwnerGetBaseAddress = nullptr;
		CSSourceInfoGetLineNumber = nullptr;
		CSSourceInfoGetPath = nullptr;
		CSSourceInfoGetRange = nullptr;
		CSSourceInfoGetSymbol = nullptr;
		FPlatformProcess::FreeDllHandle(GCoreSymbolicationHandle);
		GCoreSymbolicationHandle = nullptr;
	}
}

bool FApplePlatformSymbolication::LoadSymbolDatabaseForBinary(FString SourceFolder, FString BinaryPath, FString BinarySignature, TOptional<FString> Architecture, FApplePlatformSymbolDatabase& OutDatabase)
{
	bool bOK = FGenericPlatformSymbolication::LoadSymbolDatabaseForBinary(SourceFolder, BinaryPath, BinarySignature, Architecture, *OutDatabase.GenericDB);
	if(!bOK && GAllowApplePlatformSymbolication && (IFileManager::Get().FileSize(*BinaryPath) > 0))
	{
		CSSymbolicatorRef Symbolicator = OutDatabase.AppleDB;
		if( CSIsNull(Symbolicator) && (!Architecture.IsSet() || *Architecture == TEXT("X86-64")))
		{
			Symbolicator = CSSymbolicatorCreateWithPathAndArchitecture(TCHAR_TO_UTF8(*BinaryPath), CPU_TYPE_X86_64);
		}
        if (CSIsNull(Symbolicator) && (!Architecture.IsSet() || *Architecture == TEXT("ARM-64")))
        {
            Symbolicator = CSSymbolicatorCreateWithPathAndArchitecture(TCHAR_TO_UTF8(*BinaryPath), CPU_TYPE_ARM64);
        }
		if( !CSIsNull(Symbolicator) )
		{
			if(BinarySignature.Len())
			{
				CFUUIDRef UUID = CFUUIDCreateFromString(kCFAllocatorDefault, (CFStringRef)BinarySignature.GetNSString());
				check(UUID);
				
				CSSymbolOwnerRef SymbolOwner = CSSymbolicatorGetSymbolOwnerWithUUIDAtTime(Symbolicator, UUID, kCSNow);
				bOK = (!CSIsNull(SymbolOwner));
				OutDatabase.GenericDB->Signature = BinarySignature;
			}
			else
			{
				CSSymbolOwnerRef SymbolOwner = CSSymbolicatorGetSymbolOwner(Symbolicator);
				if(!CSIsNull(SymbolOwner))
				{
					CFUUIDRef OwnerUUID = CSSymbolOwnerGetUUID(SymbolOwner);
					CFStringRef String = CFUUIDCreateString(kCFAllocatorDefault, OwnerUUID);
					check(String);
					OutDatabase.GenericDB->Signature = FString((NSString*)String);
					CFRelease(String);
					bOK = true;
				}
			}
			
			if(bOK)
			{
				OutDatabase.AppleDB = Symbolicator;
			}
		}
	}
	return bOK;
}

bool FApplePlatformSymbolication::SaveSymbolDatabaseForBinary(FString TargetFolder, FString InName, FString BinarySignature, FApplePlatformSymbolDatabase& Database)
{
	__block bool bOK = true;
	if(GAllowApplePlatformSymbolication)
	{
		CSSymbolicatorRef Symbolicator = Database.AppleDB;
		if( !CSIsNull(Symbolicator) )
		{
			__block TMap<FString, int32> StringLookup;
			
			CSSymbolOwnerRef SymbolOwner = CSSymbolicatorGetSymbolOwner(Symbolicator);
			CFUUIDRef OwnerUUID = CSSymbolOwnerGetUUID(SymbolOwner);
			const char* OwnerName = CSSymbolOwnerGetName(SymbolOwner);
			vm_address_t BaseAddress = CSSymbolOwnerGetBaseAddress(SymbolOwner);
			
			Database.GenericDB->Name = OwnerName;
			Database.GenericDB->StringTable.Reset();
			Database.GenericDB->Symbols.Reset();
			
            if(BinarySignature.Len())
            {
                CFUUIDRef UUID = CFUUIDCreateFromString(kCFAllocatorDefault, (CFStringRef)BinarySignature.GetNSString());
                check(UUID);
                Database.GenericDB->Signature = BinarySignature;
            }
            else
            {
                    CFStringRef String = CFUUIDCreateString(kCFAllocatorDefault, OwnerUUID);
                    check(String);
                    Database.GenericDB->Signature = FString((NSString*)String);
                    CFRelease(String);
            }
			
			CSSymbolicatorForeachSymbolAtTime(Symbolicator, kCSNow, ^(CSSymbolRef Symbol){
				CSSymbolOwnerRef Owner = CSSymbolGetSymbolOwner(Symbol);
				if (CSEqual(SymbolOwner, Owner))
				{
					__block FGenericPlatformSymbolData SymbolData;
					CSRange Range = CSSymbolGetRange(Symbol);
					const char* Name = CSSymbolGetName(Symbol);
					SymbolData.Start = Range.Location - BaseAddress;
					SymbolData.Length = Range.Length;
					int32* NameIndex = StringLookup.Find(Name);
					if(NameIndex)
					{
						SymbolData.NameIdx = *NameIndex;
					}
					else
					{
						SymbolData.NameIdx = Database.GenericDB->StringTable.Num();
						StringLookup.Add(Name, SymbolData.NameIdx);
						Database.GenericDB->StringTable.Push(Name);
					}
					
					CSSymbolForeachSourceInfo(Symbol, ^(CSSourceInfoRef SourceInfo){
						int Line = CSSourceInfoGetLineNumber(SourceInfo);
						const char* Path = CSSourceInfoGetPath(SourceInfo);
						CSRange InfoRange = CSSourceInfoGetRange(SourceInfo);
						
						FGenericPlatformSymbolInfo Info;
						Info.Line = Line;
						Info.Start = InfoRange.Location - BaseAddress;
						Info.Length = InfoRange.Length;
						
						int32* PathIndex = StringLookup.Find(Path);
						if(PathIndex)
						{
							Info.PathIdx = *PathIndex;
						}
						else
						{
							Info.PathIdx = Database.GenericDB->StringTable.Num();
							StringLookup.Add(Path, Info.PathIdx);
							Database.GenericDB->StringTable.Push(Path);
						}
						
						SymbolData.SymbolInfo.Add(Info);
						
						return 0;
					});
					
					Database.GenericDB->Symbols.Add(SymbolData);
				}
				else
				{
					UE_LOGF(LogInit, Warning, "FApplePlatformSymbolication::SaveSymbolDatabaseForBinary doesn't handle Mach-O binaries/.dSYMs with multiple symbol owners!");
					bOK = false;
					return 1;
				}
				return 0;
			});
			bOK &= (Database.GenericDB->Symbols.Num() >= 1);
		}
	}
	
	if(bOK)
	{
		bOK = FGenericPlatformSymbolication::SaveSymbolDatabaseForBinary(TargetFolder, InName, *Database.GenericDB);
	}
	
	return bOK;
}

bool FApplePlatformSymbolication::SymbolInfoForStrippedSymbol(FApplePlatformSymbolDatabase const& Database, uint64 ProgramCounter, uint64 ModuleOffset, FString ModuleSignature, FProgramCounterSymbolInfo& Info)
{
	bool bOK = FGenericPlatformSymbolication::SymbolInfoForStrippedSymbol(*Database.GenericDB, ProgramCounter, ModuleOffset, ModuleSignature, Info);
	if(!bOK && GAllowApplePlatformSymbolication && !CSIsNull(Database.AppleDB) && (ModuleSignature.Len()))
	{
		CSSymbolicatorRef Symbolicator = Database.AppleDB;
	
		CFUUIDRef UUID = CFUUIDCreateFromString(kCFAllocatorDefault, (CFStringRef)ModuleSignature.GetNSString());
		check(UUID);
		
		CSSymbolOwnerRef SymbolOwner = CSSymbolicatorGetSymbolOwnerWithUUIDAtTime(Symbolicator, UUID, kCSNow);
		if(!CSIsNull(SymbolOwner))
		{
			ANSICHAR const* DylibName = CSSymbolOwnerGetName(SymbolOwner);
			FCStringAnsi::Strcpy(Info.ModuleName, DylibName);
			
			uint64 Address = ProgramCounter >= ModuleOffset ? (ProgramCounter - ModuleOffset) : ProgramCounter;
			
			vm_address_t BaseAddress = CSSymbolOwnerGetBaseAddress(SymbolOwner);
			CSSymbolRef Symbol = CSSymbolicatorGetSymbolWithAddressAtTime(Symbolicator, ((vm_address_t)Address) + BaseAddress, kCSNow);
			if(!CSIsNull(Symbol))
			{
				ANSICHAR const* FunctionName = CSSymbolGetName(Symbol);
				if(FunctionName)
				{
					FCStringAnsi::Strncpy(Info.FunctionName, FunctionName, UE_ARRAY_COUNT(Info.FunctionName));
				}
				
				CSRange Range = CSSymbolGetRange(Symbol);
				Info.SymbolDisplacement = (ProgramCounter - Range.Location);
				Info.OffsetInModule = Range.Location;
				Info.ProgramCounter = ProgramCounter;
				
				CSSourceInfoRef SymbolInfo = CSSymbolicatorGetSourceInfoWithAddressAtTime(Symbolicator, ((vm_address_t)Address) + BaseAddress, kCSNow);
				if(!CSIsNull(SymbolInfo))
				{
					Info.LineNumber = CSSourceInfoGetLineNumber(SymbolInfo);
					
					ANSICHAR const* FileName = CSSourceInfoGetPath(SymbolInfo);
					if(FileName)
					{
						FCStringAnsi::Strncpy(Info.Filename, FileName, UE_ARRAY_COUNT(Info.FunctionName));
					}
				}
				
				bOK = true;
			}
		}
	}
	
	return bOK;
}

bool FApplePlatformSymbolication::SymbolInfoForAddress(uint64 ProgramCounter, FProgramCounterSymbolInfo& out_SymbolInfo)
{
	bool bOK = false;

	// Store the PC into the info structure
	out_SymbolInfo.ProgramCounter = ProgramCounter;
	
	if (GAllowApplePlatformSymbolication)
	{
		CSSymbolicatorRef Symbolicator = CSSymbolicatorCreateWithPid(FPlatformProcess::GetCurrentProcessId());
		if(!CSIsNull(Symbolicator))
		{
			CSSourceInfoRef Symbol = CSSymbolicatorGetSourceInfoWithAddressAtTime(Symbolicator, (vm_address_t)ProgramCounter, kCSNow);
			
			if(!CSIsNull(Symbol))
			{
				out_SymbolInfo.LineNumber = CSSourceInfoGetLineNumber(Symbol);
				FGenericPlatformStackWalk::CopyVirtualPathToLocal(out_SymbolInfo.Filename, UE_ARRAY_COUNT(out_SymbolInfo.Filename), CSSourceInfoGetPath(Symbol));
				FCStringAnsi::Strncpy(out_SymbolInfo.FunctionName, CSSymbolGetName(CSSourceInfoGetSymbol(Symbol)), UE_ARRAY_COUNT(out_SymbolInfo.FunctionName));
				CSRange CodeRange = CSSourceInfoGetRange(Symbol);
				out_SymbolInfo.SymbolDisplacement = (ProgramCounter - CodeRange.Location);
				
				CSSymbolOwnerRef Owner = CSSourceInfoGetSymbolOwner(Symbol);
				if(!CSIsNull(Owner))
				{
					ANSICHAR const* DylibName = CSSymbolOwnerGetName(Owner);
					FCStringAnsi::Strncpy(out_SymbolInfo.ModuleName, DylibName, UE_ARRAY_COUNT(out_SymbolInfo.ModuleName));
					
					bOK = out_SymbolInfo.LineNumber != 0;
				}
			}
			
			CSRelease(Symbolicator);
		}
	}
	
	return bOK;
}
