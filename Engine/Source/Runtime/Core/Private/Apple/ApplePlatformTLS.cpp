// Copyright Epic Games, Inc. All Rights Reserved.

#include "Apple/ApplePlatformTLS.h"

#if PLATFORM_MAC
#include "Mac/MacSystemIncludes.h"
#elif PLATFORM_IOS
#include "IOS/IOSSystemIncludes.h"
#endif

uint32 FApplePlatformTLS::GetCurrentThreadId()
{
	return (uint32)pthread_mach_thread_np(pthread_self());
}

uint32 FApplePlatformTLS::AllocTlsSlot()
{
	// allocate a per-thread mem slot
	pthread_key_t SlotKey = 0;
	if (pthread_key_create(&SlotKey, NULL) != 0)
	{
		SlotKey = InvalidTlsSlot;  // matches the Windows TlsAlloc() retval.
	}
	return SlotKey;
}

void FApplePlatformTLS::SetTlsValue(uint32 SlotIndex, void* Value)
{
	pthread_setspecific((pthread_key_t)SlotIndex, Value);
}

void* FApplePlatformTLS::GetTlsValue(uint32 SlotIndex)
{
	return pthread_getspecific((pthread_key_t)SlotIndex);
}

void FApplePlatformTLS::FreeTlsSlot(uint32 SlotIndex)
{
	pthread_key_delete((pthread_key_t)SlotIndex);
}
