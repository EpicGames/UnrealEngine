// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ApplePlatformTime.mm: Apple implementations of time functions
=============================================================================*/

#include "Apple/ApplePlatformTime.h"
#include "HAL/PlatformTime.h"
#if PLATFORM_MAC
#include "Mac/MacSystemIncludes.h"
#elif PLATFORM_IOS
#include "IOS/IOSSystemIncludes.h"
#endif
#include "Misc/AssertionMacros.h"
#include "CoreGlobals.h"

#include "CoreTypes.h"



namespace
{
	FORCEINLINE double TimeValToSecond(timeval & tv)
	{
		return static_cast<double>(tv.tv_sec) + static_cast<double>(tv.tv_usec) / 1e6;
	}
}

double FApplePlatformTime::InitTiming(void)
{
	// Time base is in nano seconds.
	mach_timebase_info_data_t Info;
	verify( mach_timebase_info( &Info ) == 0 );
	SecondsPerCycle = 1e-9 * (double)Info.numer / (double)Info.denom;
	SecondsPerCycle64 = 1e-9 * (double)Info.numer / (double)Info.denom;
	return FPlatformTime::Seconds();
}

double FApplePlatformTime::Seconds()
{
	uint64 Cycles = mach_absolute_time();
	// Add big number to make bugs apparent where return value is being passed to float
	return static_cast<double>(Cycles) * GetSecondsPerCycle() + GetSecondsTimeOffset();
}

uint32 FApplePlatformTime::Cycles()
{
	uint64 Cycles = mach_absolute_time();
	return Cycles;
}

uint64 FApplePlatformTime::Cycles64()
{
	uint64 Cycles = mach_absolute_time();
	return Cycles;
}

FCPUTime FApplePlatformTime::GetCPUTime()
{
	// minimum delay between checks to minimize overhead (and also match Windows version)
	const double MinDelayBetweenChecks = 0.025;
	
	// last time we checked the timer
	static double PreviousUpdateTime = GStartTime;
	// last user + system time
	static double PreviousSystemAndUserProcessTime = 0.0;

	// last CPU utilization
	static float CurrentCpuUtilization = 0.0f;
	// last CPU utilization (per core)
	static float CurrentCpuUtilizationNormalized = 0.0f;

	const double CurrentTime = Seconds();

	// see if we need to update the values
	double TimeSinceLastUpdate = CurrentTime - PreviousUpdateTime ;
	if (TimeSinceLastUpdate >= MinDelayBetweenChecks)
	{
		struct rusage Usage;
		if (0 == getrusage(RUSAGE_SELF, &Usage))
		{				
			const double CurrentSystemAndUserProcessTime = TimeValToSecond(Usage.ru_utime) + TimeValToSecond(Usage.ru_stime); // holds all usages on all cores
			const double CpuTimeDuringPeriod = CurrentSystemAndUserProcessTime - PreviousSystemAndUserProcessTime;

			double CurrentCpuUtilizationHighPrec = CpuTimeDuringPeriod / TimeSinceLastUpdate * 100.0;

			// recalculate the values
			CurrentCpuUtilizationNormalized = static_cast< float >( CurrentCpuUtilizationHighPrec / static_cast< double >( FPlatformMisc::NumberOfCoresIncludingHyperthreads() ) );
			CurrentCpuUtilization = static_cast< float >( CurrentCpuUtilizationHighPrec );

			// update previous
			PreviousSystemAndUserProcessTime = CurrentSystemAndUserProcessTime;
			PreviousUpdateTime = CurrentTime;
		}
	}

	return FCPUTime(CurrentCpuUtilizationNormalized, CurrentCpuUtilization);
}
