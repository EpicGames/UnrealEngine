// Copyright Epic Games, Inc. All Rights Reserved.

#include "Apple/AppleRecursiveMutex.h"

namespace UE::Private
{

static LockImplementation LockRuntimeDispatch;
static LockImplementation LockFallback;

CORE_API LockImplementation* LockWithFlags = &LockRuntimeDispatch;

static void LockRuntimeDispatch(os_unfair_lock_t lock, os_unfair_lock_flags_t flags)
{
	//
	// os_unfair_lock_lock_with_flags is available on: 
	//   iOS 18.0+, iPadOS 18.0+, Mac Catalyst 18.0+, macOS 15.0+, tvOS 18.0+, visionOS 2.0+, watchOS 11.0+
	//
	// It is a weak import, so will be nullptr on older OS's. Hence, we can do a
	// runtime dispatch on the first lock call.
	//
	LockWithFlags = os_unfair_lock_lock_with_flags ? os_unfair_lock_lock_with_flags : &LockFallback;
	LockWithFlags(lock, flags);
}

static void LockFallback(os_unfair_lock_t lock, os_unfair_lock_flags_t flags)
{
	(void)flags;
	os_unfair_lock_lock(lock);
}

} // namespace UE::Private
