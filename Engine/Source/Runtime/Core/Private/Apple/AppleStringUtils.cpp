// Copyright Epic Games, Inc. All Rights Reserved.

#include "Apple/AppleStringUtils.h"
#if PLATFORM_APPLE

#include "Containers/UnrealString.h"

#include "Containers/StringConv.h"

@implementation NSString (FString_Extensions)

+ (NSString*) stringWithTCHARString:(const TCHAR*)MyTCHARString
{
	return [NSString stringWithCString:TCHAR_TO_UTF8(MyTCHARString) encoding:NSUTF8StringEncoding];
}

+ (NSString*) stringWithFString:(const FString&)InFString
{
	return [NSString stringWithTCHARString:*InFString];
}


@end

NSString* FAppleStringUtils::ConvertToNSString(const FString& InString)
{
	NSString* OutString = (NSString*)(InString.GetCFString());
	[OutString autorelease];
        
	return OutString;
}

FString FAppleStringUtils::ConvertToFString(const  NSString* InNSString)
{
	return FString((__bridge CFStringRef)InNSString);
}

#endif