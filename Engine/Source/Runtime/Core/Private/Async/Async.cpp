// Copyright Epic Games, Inc. All Rights Reserved.

#include "Async/Async.h"

#include "HAL/ThreadSafeCounter.h"


/* Private types
 *****************************************************************************/

/** Graph task for simple fire-and-forget asynchronous functions. */
class FAsyncGraphTask
	: public FAsyncGraphTaskBase
{
public:

	/** Creates and initializes a new instance. */
	FAsyncGraphTask(ENamedThreads::Type InDesiredThread, TUniqueFunction<void()>&& InFunction)
		: DesiredThread(InDesiredThread)
		, Function(MoveTemp(InFunction))
	{ }

	/** Performs the actual task. */
	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
	{
		Function();
	}

	/** Returns the name of the thread that this task should run on. */
	ENamedThreads::Type GetDesiredThread()
	{
		return DesiredThread;
	}

private:

	/** The thread to execute the function on. */
	ENamedThreads::Type DesiredThread;

	/** The function to execute on the Task Graph. */
	TUniqueFunction<void()> Function;
};


/* Global functions
 *****************************************************************************/

CORE_API int32 FAsyncThreadIndex::GetNext()
{
	static FThreadSafeCounter ThreadIndex;
	return ThreadIndex.Add(1);
}

void AsyncTask(ENamedThreads::Type Thread, TUniqueFunction<void()> Function)
{
	TGraphTask<FAsyncGraphTask>::CreateTask().ConstructAndDispatchWhenReady(Thread, MoveTemp(Function));
}
