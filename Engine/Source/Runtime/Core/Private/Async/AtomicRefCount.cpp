// Copyright Epic Games, Inc. All Rights Reserved.

#include "Async/AtomicRefCount.h"
#include "Misc/AssertionMacros.h"

void UE::Private::CheckRefCountIsNonZero()
{
	check(!"Release() was called on an object which is already at a zero refcount.");
}
