// Copyright Epic Games, Inc. All Rights Reserved.

#include "Async/Fundamental/Scheduler.h"
#include "Async/Fundamental/Task.h"
#include "Async/TaskTrace.h"
#include "Async/UniqueLock.h"
#include "Containers/ConsumeAllMpmcQueue.h"
#include "Logging/LogMacros.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "HAL/MallocAnsi.h"
#include "HAL/PlatformMallocCrash.h"
#include "HAL/RunnableThread.h"
#include "Misc/ScopeLock.h"
#include "Misc/CommandLine.h"
#include "Misc/Fork.h"
#include "CoreGlobals.h"

extern CORE_API bool GTaskGraphUseDynamicPrioritization;
extern CORE_API float GTaskGraphOversubscriptionRatio;
extern CORE_API bool GTaskGraphUseDynamicThreadCreation;

CSV_DEFINE_CATEGORY(Scheduler, true);

namespace LowLevelTasks
{
	DEFINE_LOG_CATEGORY(LowLevelTasks);

	thread_local FSchedulerTls::FTlsValuesHolder FSchedulerTls::TlsValuesHolder;
	thread_local FTask* FTask::ActiveTask = nullptr;
	thread_local bool Private::FOversubscriptionTls::bIsOversubscriptionAllowed = false;

	struct FTlsValuesAllocator
	{
		typedef SIZE_T SizeType;
		enum { NeedsElementType = false };

		static void* Malloc(SIZE_T Count, uint32 Alignment = DEFAULT_ALIGNMENT)
		{
			// SchedulerTls are allocated very early at thread creation and we need an allocator
			// that is OK with being called that early. We use AnsiMallocUntracked. Attempting to add tracking
			// of the pointers causes problems with some other systems that inject themselves into thread creation,
			// such as RenderDoc. Do not change the allocation function we are using without testing these systems:
			//     RenderDoc, LowLevelMemTracker.
			return AnsiMallocUntracked(Count, Alignment);
		}

		static void Free(void* Ptr)
		{
			AnsiFreeUntracked(Ptr);
		}
	};

	void* FSchedulerTls::FTlsValues::operator new(size_t Size)
	{
		return FTlsValuesAllocator::Malloc(Size, PLATFORM_CACHE_LINE_SIZE);
	}

	void FSchedulerTls::FTlsValues::operator delete(void* Ptr)
	{
		FTlsValuesAllocator::Free(Ptr);
	}

	class FSchedulerTls::FImpl
	{
	public:
		static UE::FMutex                 ThreadTlsValuesMutex;
		static FSchedulerTls::FTlsValues* ThreadTlsValues;

		static UE::TConsumeAllMpmcQueue<FSchedulerTls::FTlsValues*, FTlsValuesAllocator> PendingInsertTlsValues;
		static UE::TConsumeAllMpmcQueue<FSchedulerTls::FTlsValues*, FTlsValuesAllocator> PendingDeleteTlsValues;

		static void ProcessPendingTlsValuesNoLock()
		{
			PendingInsertTlsValues.ConsumeAllLifo(
				[](FTlsValues* TlsValues)
				{
						TlsValues->LinkHead(ThreadTlsValues);
				}
			);

			PendingDeleteTlsValues.ConsumeAllLifo(
				[](FTlsValues* TlsValues)
				{
						TlsValues->Unlink();
						delete TlsValues;
				}
			);
		}
	};

	UE::FMutex                 FSchedulerTls::FImpl::ThreadTlsValuesMutex;
	FSchedulerTls::FTlsValues* FSchedulerTls::FImpl::ThreadTlsValues = nullptr;

	UE::TConsumeAllMpmcQueue<FSchedulerTls::FTlsValues*, FTlsValuesAllocator> FSchedulerTls::FImpl::PendingInsertTlsValues;
	UE::TConsumeAllMpmcQueue<FSchedulerTls::FTlsValues*, FTlsValuesAllocator> FSchedulerTls::FImpl::PendingDeleteTlsValues;

	FSchedulerTls::FTlsValuesHolder::FTlsValuesHolder()
	{
		// Avoid a deadlock on threads being spun up or down during a crash
		if (!FPlatformMallocCrash::IsActive())
		{
			TlsValues = new FTlsValues();

			if (FImpl::ThreadTlsValuesMutex.TryLock())
			{
				FImpl::ProcessPendingTlsValuesNoLock();
				TlsValues->LinkHead(FImpl::ThreadTlsValues);
				FImpl::ThreadTlsValuesMutex.Unlock();
			}
			else
			{
				FImpl::PendingInsertTlsValues.ProduceItem(TlsValues);
			}
		}
	}

	FSchedulerTls::FTlsValuesHolder::~FTlsValuesHolder()
	{
		if (TlsValues && !FPlatformMallocCrash::IsActive())
		{
			if (FImpl::ThreadTlsValuesMutex.TryLock())
			{
				FImpl::ProcessPendingTlsValuesNoLock();
				TlsValues->Unlink();
				FImpl::ThreadTlsValuesMutex.Unlock();
				delete TlsValues;
			}
			else
			{
				FImpl::PendingDeleteTlsValues.ProduceItem(TlsValues);
			}

			TlsValues = nullptr;
		}
	}

	FORCENOINLINE bool& Private::FOversubscriptionTls::GetIsOversubscriptionAllowedRef()
	{
		return bIsOversubscriptionAllowed;
	}

	FORCENOINLINE void FOversubscriptionScope::TryIncrementOversubscription()
	{
		if (Private::FOversubscriptionTls::IsOversubscriptionAllowed())
		{
		#if CPUPROFILERTRACE_ENABLED
			if (CpuChannel)
			{
				static uint32 OversubscriptionTraceId = FCpuProfilerTrace::OutputEventType("Oversubscription");
				FCpuProfilerTrace::OutputBeginEvent(OversubscriptionTraceId);
				bCpuBeginEventEmitted = true;
			}
		#endif

			bIncrementOversubscriptionEmitted = true;
			FScheduler::Get().IncrementOversubscription();
		}
	}

	FORCENOINLINE void FOversubscriptionScope::DecrementOversubscription()
	{
		FScheduler::Get().DecrementOversubscription();
		bIncrementOversubscriptionEmitted = false;

	#if CPUPROFILERTRACE_ENABLED
		if (bCpuBeginEventEmitted)
		{
			FCpuProfilerTrace::OutputEndEvent();
			bCpuBeginEventEmitted = false;
		}
	#endif
	}

	FORCENOINLINE FSchedulerTls::FTlsValues& FSchedulerTls::GetTlsValuesRef()
	{
		return *TlsValuesHolder.TlsValues;
	}

	FScheduler FScheduler::Singleton;

	TUniquePtr<FThread> FScheduler::CreateWorker(uint32 WorkerId, const TCHAR* Name, bool bPermitBackgroundWork, FThread::EForkable IsForkable, Private::FWaitEvent* ExternalWorkerEvent, FSchedulerTls::FLocalQueueType* ExternalWorkerLocalQueue, EThreadPriority Priority, uint64 InAffinity, uint32 StackSize)
	{
		const uint32 WaitTimes[8] = { 719, 991, 1361, 1237, 1597, 953, 587, 1439 };
		uint32 WaitTime = WaitTimes[WorkerId % 8];
		uint64 ThreadAffinityMask = FPlatformAffinity::GetTaskGraphThreadMask();
		if (bPermitBackgroundWork && FPlatformAffinity::GetTaskGraphBackgroundTaskMask() != 0xFFFFFFFFFFFFFFFF)
		{
			ThreadAffinityMask = FPlatformAffinity::GetTaskGraphBackgroundTaskMask();
		}
		if (InAffinity)
		{
			// we can override the affinity!
			ThreadAffinityMask = InAffinity;
		}

		const FProcessorGroupDesc& ProcessorGroups = FPlatformMisc::GetProcessorGroupDesc();
		int32 CpuGroupCount = ProcessorGroups.NumProcessorGroups;
		uint16 CpuGroup = 0;

		//offset the first set of workers to leave space for Game, RHI and Renderthread.
		uint64 GroupWorkerId = WorkerId + 2;
		for (uint16 GroupIndex = 0; GroupIndex < CpuGroupCount; GroupIndex++)
		{
			CpuGroup = GroupIndex;

			uint32 CpusInGroup = FMath::CountBits64(ProcessorGroups.ThreadAffinities[GroupIndex]);
			if (GroupWorkerId < CpusInGroup)
			{
				if (CpuGroup != 0) // don't pin larger groups workers to a core and leave first group as is for legacy reasons
				{
					ThreadAffinityMask = MAX_uint64;
				}
				break;
			}
			GroupWorkerId -= CpusInGroup;
		}
		
		return MakeUnique<FThread>
		(
			Name,
			[this, ExternalWorkerEvent, ExternalWorkerLocalQueue, WaitTime, bPermitBackgroundWork]
			{ 
				WorkerMain(ExternalWorkerEvent, ExternalWorkerLocalQueue, WaitTime, bPermitBackgroundWork);
			}, StackSize, Priority, FThreadAffinity{ ThreadAffinityMask & ProcessorGroups.ThreadAffinities[CpuGroup], CpuGroup }, IsForkable
		);
	}

	void FScheduler::StartWorkers(uint32 NumForegroundWorkers, uint32 NumBackgroundWorkers, FThread::EForkable IsForkable, EThreadPriority InWorkerPriority,  EThreadPriority InBackgroundPriority, uint64 InWorkerAffinity, uint64 InBackgroundAffinity, uint32 InWorkersStackSize)
	{
		// It always been a given that only the game thread should start workers so just add validation.
		check(IsInGameThread());
		int32 Value = 0;
		if (FParse::Value(FCommandLine::Get(), TEXT("TaskGraphUseDynamicPrioritization="), Value))
		{
			GTaskGraphUseDynamicPrioritization = Value != 0;
		}

		if (FParse::Value(FCommandLine::Get(), TEXT("TaskGraphUseDynamicThreadCreation="), Value))
		{
			GTaskGraphUseDynamicThreadCreation = Value != 0;
		}

		if (NumForegroundWorkers == 0 && NumBackgroundWorkers == 0)
		{
			NumForegroundWorkers = FMath::Max<int32>(1, FMath::Min<int32>(2, FPlatformMisc::NumberOfWorkerThreadsToSpawn() - 1));
			NumBackgroundWorkers = FMath::Max<int32>(1, FPlatformMisc::NumberOfWorkerThreadsToSpawn() - NumForegroundWorkers);
		}

		WorkerPriority = InWorkerPriority;
		BackgroundPriority = InBackgroundPriority;

		if (InWorkerAffinity)
		{
			WorkerAffinity = InWorkerAffinity;
		}
		if (InBackgroundAffinity)
		{
			BackgroundAffinity = InBackgroundAffinity;
		}

		WorkerStackSize = InWorkersStackSize;

		const bool bSupportsMultithreading = FPlatformProcess::SupportsMultithreading() || FForkProcessHelper::IsForkedMultithreadInstance();

		uint32 OldActiveWorkers = ActiveWorkers.load(std::memory_order_relaxed);
		if (OldActiveWorkers == 0 && bSupportsMultithreading && ActiveWorkers.compare_exchange_strong(OldActiveWorkers, NumForegroundWorkers + NumBackgroundWorkers, std::memory_order_relaxed))
		{
			UE::TUniqueLock Lock(WorkerThreadsCS);
			check(!WorkerThreads);
			check(!WorkerLocalQueues.Num());
			check(!WorkerEvents.Num());
			check(NextWorkerId == 0);
			ForegroundCreationIndex = 0;
			BackgroundCreationIndex = 0;

			const float OversubscriptionRatio = FMath::Max(1.0f, GTaskGraphOversubscriptionRatio);
			const int32 MaxForegroundWorkers = FMath::CeilToInt(float(NumForegroundWorkers) * OversubscriptionRatio);
			const int32 MaxBackgroundWorkers = FMath::CeilToInt(float(NumBackgroundWorkers) * OversubscriptionRatio);
			const int32 MaxWorkers = MaxForegroundWorkers + MaxBackgroundWorkers;
			const EThreadPriority ActualBackgroundPriority = GTaskGraphUseDynamicPrioritization ? WorkerPriority : BackgroundPriority;

			if (GameThreadLocalQueue == nullptr)
			{
				GameThreadLocalQueue = MakeUnique<FSchedulerTls::FLocalQueueType>(QueueRegistry, Private::ELocalQueueType::EForeground);
			}
			FSchedulerTls::GetTlsValuesRef().LocalQueue = GameThreadLocalQueue.Get();

			WorkerEvents.SetNum(MaxWorkers);
			WorkerLocalQueues.SetNum(MaxWorkers);
			WorkerThreads.Reset(new std::atomic<FThread*>[MaxWorkers]);

			auto CreateThread =
				[this, IsForkable](Private::ELocalQueueType LocalQueueType, const TCHAR* ThreadGroup, const TCHAR* Prefix, std::atomic<int32>& CreationIndex, int32 NumWorkers, int32 NumMaxWorkers, EThreadPriority Priority, uint64 Affinity, uint32 StackSize)
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(CreateWorkerThread);
					LLM_SCOPE_BYNAME(TEXT("EngineMisc/WorkerThreads"));

					// Thread creation can end up waiting, we don't want to recursively oversubscribe if that happens.
					Private::FOversubscriptionAllowedScope _(false); 

					const int32 LocalCreationIndex = CreationIndex++;
					check(LocalCreationIndex < NumMaxWorkers);
					const bool bIsStandbyWorker = LocalCreationIndex >= NumWorkers;
					FString WorkerName;
					if (bIsStandbyWorker)
					{
						WorkerName = FString::Printf(TEXT("%s Worker (Standby #%d)"), Prefix, LocalCreationIndex - NumWorkers);
					}
					else
					{
						WorkerName = FString::Printf(TEXT("%s Worker #%d"), Prefix, LocalCreationIndex);
					}

					uint32 WorkerId = NextWorkerId++;
					UE::Trace::ThreadGroupBegin(ThreadGroup);
					WorkerLocalQueues[WorkerId].Init(QueueRegistry, LocalQueueType);
					WorkerEvents[WorkerId].bIsStandby = bIsStandbyWorker;
					WorkerThreads[WorkerId] =
						CreateWorker(
							WorkerId,
							*WorkerName,
							LocalQueueType == Private::ELocalQueueType::EBackground, /* bPermitBackgroundWork */
							IsForkable,
							&WorkerEvents[WorkerId],
							&WorkerLocalQueues[WorkerId],
							Priority,
							Affinity,
							StackSize).Release();

					UE::Trace::ThreadGroupEnd();
				};

			TFunction<void()> ForegroundCreateThread =
				[this, CreateThread, NumWorkers = NumForegroundWorkers, MaxWorkers = MaxForegroundWorkers]()
				{
					CreateThread(Private::ELocalQueueType::EForeground, TEXT("Foreground Workers"), TEXT("Foreground"), ForegroundCreationIndex, NumWorkers, MaxWorkers, WorkerPriority, WorkerAffinity, WorkerStackSize);
				};

			TFunction<void()> BackgroundCreateThread =
				[this, CreateThread, NumWorkers = NumBackgroundWorkers, MaxWorkers = MaxBackgroundWorkers, ActualBackgroundPriority]()
				{
					CreateThread(Private::ELocalQueueType::EBackground, TEXT("Background Workers"), TEXT("Background"), BackgroundCreationIndex, NumWorkers, MaxWorkers, ActualBackgroundPriority, BackgroundAffinity, WorkerStackSize);
				};

			WaitingQueue[0].Init(NumForegroundWorkers, MaxForegroundWorkers, ForegroundCreateThread, GTaskGraphUseDynamicThreadCreation ? 0 : MaxForegroundWorkers /* ActiveThreadCount */);
			WaitingQueue[1].Init(NumBackgroundWorkers, MaxBackgroundWorkers, BackgroundCreateThread, GTaskGraphUseDynamicThreadCreation ? 0 : MaxBackgroundWorkers /* ActiveThreadCount */);

			// Precreate all the threads if dynamic thread creation is not activated.
			if (!GTaskGraphUseDynamicThreadCreation)
			{
				for (int32 Index = 0; Index < MaxForegroundWorkers; Index++)
				{
					ForegroundCreateThread();
				}

				for (int32 Index = 0; Index < MaxBackgroundWorkers; Index++)
				{
					BackgroundCreateThread();
				}
			}
			else if (TemporaryShutdown)
			{
				// Since the global queue is not drained during temporary shutdown, kick threads
				// here so we can continue work if there was any tasks left when we stopped the workers.
				WaitingQueue[0].Notify();
				WaitingQueue[1].Notify();
			}
		}
	}
	
	bool FScheduler::IsOversubscriptionLimitReached(ETaskPriority TaskPriority) const
	{
		const bool bIsBackgroundTask = TaskPriority >= ETaskPriority::ForegroundCount;
		if (bIsBackgroundTask)
		{
			return 
				WaitingQueue[1].IsOversubscriptionLimitReached();
		}
		else
		{
			// Since we are allowing background thread to run foreground task we need both waiting queue
			// to reach their limit to consider that priority's limit reached.
			return 
				WaitingQueue[0].IsOversubscriptionLimitReached() &&
				WaitingQueue[1].IsOversubscriptionLimitReached();
		}
	}
	
	FOversubscriptionLimitReached& FScheduler::GetOversubscriptionLimitReachedEvent()
	{
		return OversubscriptionLimitReachedEvent;
	}

	inline FTask* FScheduler::ExecuteTask(FTask* InTask)
	{
		FTask* ParentTask = FTask::ActiveTask;
		FTask::ActiveTask = InTask;
		FTask* OutTask;

		if (!InTask->IsBackgroundTask())
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(ExecuteForegroundTask);
			OutTask = InTask->ExecuteTask();
		}
		else
		{
			// Dynamic priority only enables for root task when we're not inside a named thread (i.e. GT, RT)
			const bool bSkipPriorityChange = ParentTask || !GTaskGraphUseDynamicPrioritization || !FSchedulerTls::IsWorkerThread() || InTask->WasCanceledOrIsExpediting();

			FRunnableThread* RunnableThread = nullptr;
			if (!bSkipPriorityChange)
			{
				// We assume all threads executing tasks are RunnableThread and this can't be null or it will crash. 
				// Which is fine since we want to know about it sooner rather than later.
				RunnableThread = FRunnableThread::GetRunnableThread();

				checkSlow(RunnableThread && RunnableThread->GetThreadPriority() == WorkerPriority);

				TRACE_CPUPROFILER_EVENT_SCOPE(LowerThreadPriority);
				RunnableThread->SetThreadPriority(BackgroundPriority);
			}

			{
				TRACE_CPUPROFILER_EVENT_SCOPE(ExecuteBackgroundTask);
				OutTask = InTask->ExecuteTask();
			}

			if (!bSkipPriorityChange)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(RaiseThreadPriority);
				RunnableThread->SetThreadPriority(WorkerPriority);
			}
		}

		FTask::ActiveTask = ParentTask;
		return OutTask;
	}

	void FScheduler::StopWorkers(bool bDrainGlobalQueue)
	{
		// It always been a given that only the game thread should stop workers so just add validation.
		check(IsInGameThread());
		uint32 OldActiveWorkers = ActiveWorkers.load(std::memory_order_relaxed);
		if (OldActiveWorkers != 0 && ActiveWorkers.compare_exchange_strong(OldActiveWorkers, 0, std::memory_order_relaxed))
		{
			UE::TUniqueLock Lock(WorkerThreadsCS);

			WaitingQueue[0].StartShutdown();
			WaitingQueue[1].StartShutdown();

			// We wait on threads to exit, once we're done with that
			// it means no more threads can possibly get created.
			for (std::atomic<FThread*>& ThreadEntry : TArrayView<std::atomic<FThread*>>(WorkerThreads.Get(), WorkerLocalQueues.Num()))
			{
				if (FThread* Thread = ThreadEntry.exchange(nullptr))
				{
					Thread->Join();
					delete Thread;
				}
			}

			WaitingQueue[0].FinishShutdown();
			WaitingQueue[1].FinishShutdown();

			GameThreadLocalQueue.Reset();
			FSchedulerTls::GetTlsValuesRef().LocalQueue = nullptr;

			NextWorkerId = 0;
			WorkerThreads.Reset();
			WorkerLocalQueues.Reset();
			WorkerEvents.Reset();

			if (bDrainGlobalQueue)
			{
				for (FTask* Task = QueueRegistry.DequeueGlobal(); Task != nullptr; Task = QueueRegistry.DequeueGlobal())
				{
					while(Task)
					{
						if ((Task = ExecuteTask(Task)) != nullptr)
						{
							verifySlow(Task->TryPrepareLaunch());
						}
					}
				}
			}

			QueueRegistry.Reset();
		}
	}

	bool FSchedulerTls::HasPendingWakeUp() const
	{
		UE::TUniqueLock ScopedLock(FImpl::ThreadTlsValuesMutex);
		FImpl::ProcessPendingTlsValuesNoLock();

#if PLATFORM_SUPPORTS_ASYMMETRIC_FENCES
		// Heavy barrier since bPendingWakeUp is only written to with a relaxed write, we need all cores to flush their store buffer to memory
		FPlatformMisc::AsymmetricThreadFenceHeavy();
		constexpr std::memory_order MemoryOrder = std::memory_order_relaxed;
#else
		constexpr std::memory_order MemoryOrder = std::memory_order_acquire;
#endif

		for (FTlsValues::TConstIterator It(FImpl::ThreadTlsValues); It; It.Next())
		{
			const FTlsValues& TlsValue = *It;
			if (TlsValue.ActiveScheduler != this &&
				TlsValue.bPendingWakeUp.load(MemoryOrder))
			{
				return true;
			}
		}

		return false;
	}

	void FScheduler::RestartWorkers(uint32 NumForegroundWorkers, uint32 NumBackgroundWorkers, FThread::EForkable IsForkable, EThreadPriority InWorkerPriority, EThreadPriority InBackgroundPriority, uint64 InWorkerAffinity, uint64 InBackgroundAffinity, uint32 InWorkersStackSize)
	{
		UE::TUniqueLock Lock(WorkerThreadsCS);
		TemporaryShutdown.store(true, std::memory_order_release);
		while (HasPendingWakeUp())
		{
			FPlatformProcess::Yield();
		}
		const bool bDrainGlobalQueue = false;
		StopWorkers(bDrainGlobalQueue);
		StartWorkers(NumForegroundWorkers, NumBackgroundWorkers, IsForkable, InWorkerPriority, InBackgroundPriority, InWorkerAffinity, InBackgroundAffinity, InWorkersStackSize);
		TemporaryShutdown.store(false, std::memory_order_release);
	}

	void FScheduler::LaunchInternal(FTask& Task, EQueuePreference QueuePreference, bool bWakeUpWorker)
	{
		if (ActiveWorkers.load(std::memory_order_relaxed) || TemporaryShutdown.load(std::memory_order_acquire))
		{
			FTlsValues& LocalTlsValues = GetTlsValuesRef();

			const bool bIsBackgroundTask = Task.IsBackgroundTask();
			const bool bIsBackgroundWorker = LocalTlsValues.IsBackgroundWorker();
			const bool bIsStandbyWorker = LocalTlsValues.IsStandbyWorker();
			FSchedulerTls::FLocalQueueType* const CachedLocalQueue = LocalTlsValues.LocalQueue;

			// Standby workers always enqueue to the global queue and perform wakeup
			// as they can go to sleep whenever the oversubscription period is done
			// and we don't want that to happen without another thread picking up
			// this task.
			if ((bIsBackgroundTask && !bIsBackgroundWorker) || bIsStandbyWorker)
			{
				QueuePreference = EQueuePreference::GlobalQueuePreference;

				// Always wake up a worker if we're scheduling a background task from a foreground thread
				// since foreground threads are not allowed to process them.
				bWakeUpWorker = true;
			}
			else
			{
				bWakeUpWorker |= CachedLocalQueue == nullptr;
			}

			// Always force local queue usage when launching from the game thread to minimize cost.
			if (CachedLocalQueue && CachedLocalQueue == GameThreadLocalQueue.Get())
			{
				QueuePreference = EQueuePreference::LocalQueuePreference;
				bWakeUpWorker = true; // The game thread is never pumping its local queue directly, need to always perform a wakeup.
			}

			if (CachedLocalQueue && QueuePreference != EQueuePreference::GlobalQueuePreference)
			{
				CachedLocalQueue->Enqueue(&Task, uint32(Task.GetPriority()));
			}
			else
			{
				QueueRegistry.Enqueue(&Task, uint32(Task.GetPriority()));
			}

			if (bWakeUpWorker)
			{
#if PLATFORM_SUPPORTS_ASYMMETRIC_FENCES
				constexpr std::memory_order MemoryOrder = std::memory_order_relaxed;
#else
				constexpr std::memory_order MemoryOrder = std::memory_order_seq_cst;
#endif
				// We don't need to pay this cost for worker threads because we already manage their shutdown gracefully.
				const bool bExternalThread = LocalTlsValues.ActiveScheduler != this || LocalTlsValues.WorkerType == EWorkerType::None;
				if (bExternalThread)
				{
#if PLATFORM_SUPPORTS_ASYMMETRIC_FENCES
					FPlatformMisc::AsymmetricThreadFenceLight();
#endif

					LocalTlsValues.bPendingWakeUp.store(true, MemoryOrder);

#if PLATFORM_SUPPORTS_ASYMMETRIC_FENCES
					FPlatformMisc::AsymmetricThreadFenceLight();
#endif

					if (TemporaryShutdown.load(std::memory_order_acquire))
					{
						LocalTlsValues.bPendingWakeUp.store(false, MemoryOrder);
						return;
					}
				}

				if (!WakeUpWorker(bIsBackgroundTask) && !bIsBackgroundTask)
				{
					WakeUpWorker(true);
				}

				if (bExternalThread)
				{
#if PLATFORM_SUPPORTS_ASYMMETRIC_FENCES
					FPlatformMisc::AsymmetricThreadFenceLight();
#endif

					LocalTlsValues.bPendingWakeUp.store(false, MemoryOrder);

#if PLATFORM_SUPPORTS_ASYMMETRIC_FENCES
					FPlatformMisc::AsymmetricThreadFenceLight();
#endif
				}
			}
		}
		else
		{
			FTask* TaskPtr = &Task;
			while (TaskPtr)
			{
				if ((TaskPtr = ExecuteTask(TaskPtr)) != nullptr)
				{
					verifySlow(TaskPtr->TryPrepareLaunch());
				}
			}
		}
	}

	void FScheduler::IncrementOversubscription()
	{
		FSchedulerTls::EWorkerType LocalWorkerType = FSchedulerTls::GetTlsValuesRef().WorkerType;

		if (LocalWorkerType != EWorkerType::None)
		{
			// The goal is to minimize the amount of wait in the worker tasks, this will help drive the
			// total number of oversubscription down and show any regressions.
			CSV_CUSTOM_STAT(Scheduler, Oversubscription, 1, ECsvCustomStatOp::Accumulate);

			const bool bPermitBackgroundWork = LocalWorkerType == FSchedulerTls::EWorkerType::Background;
			WaitingQueue[bPermitBackgroundWork].IncrementOversubscription();
		}
	}

	void FScheduler::DecrementOversubscription()
	{
		FSchedulerTls::EWorkerType LocalWorkerType = FSchedulerTls::GetTlsValuesRef().WorkerType;

		if (LocalWorkerType != EWorkerType::None)
		{
			const bool bPermitBackgroundWork = LocalWorkerType == FSchedulerTls::EWorkerType::Background;
			WaitingQueue[bPermitBackgroundWork].DecrementOversubscription();
		}
	}

#if PLATFORM_DESKTOP || !IS_MONOLITHIC
	const FTask* FTask::GetActiveTask()
	{
		return ActiveTask;
	}
#endif

	bool FSchedulerTls::IsWorkerThread() const
	{
		FTlsValues& LocalTlsValues = FSchedulerTls::GetTlsValuesRef();
		return LocalTlsValues.WorkerType != FSchedulerTls::EWorkerType::None && LocalTlsValues.ActiveScheduler == this;
	}

	template<typename QueueType, FTask* (QueueType::*DequeueFunction)(bool), bool bIsStandbyWorker>
	bool FScheduler::TryExecuteTaskFrom(Private::FWaitEvent* WaitEvent, QueueType* Queue, Private::FOutOfWork& OutOfWork, bool bPermitBackgroundWork)
	{
		bool AnyExecuted = false;

		FTask* Task = (Queue->*DequeueFunction)(bPermitBackgroundWork);
		while (Task)
		{
			checkSlow(FTask::ActiveTask == nullptr);

			if (OutOfWork.Stop())
			{
				// Standby workers don't need cancellation, this logic doesn't apply to them.
				if constexpr (bIsStandbyWorker == false)
				{
					// CancelWait will tell us if we need to start a new worker to replace
					// a potential wakeup we might have consumed during the cancellation.
					if (WaitingQueue[bPermitBackgroundWork].CancelWait(WaitEvent))
					{
						if (!WakeUpWorker(bPermitBackgroundWork) && !GetTlsValuesRef().IsBackgroundWorker())
						{
							WakeUpWorker(!bPermitBackgroundWork);
						}
					}
				}
			}

			AnyExecuted = true;

			// Executing a task can return a continuation.
			if ((Task = ExecuteTask(Task)) != nullptr)
			{
				verifySlow(Task->TryPrepareLaunch());
			}
		}
		return AnyExecuted;
	}

	template <typename LocalQueueType>
	class TCombinedQueue
	{
		LocalQueueType* GameThreadLocalQueue;
		LocalQueueType* WorkerLocalQueue;
	public:
		TCombinedQueue(LocalQueueType* InGameThreadLocalQueue, LocalQueueType* InWorkerLocalQueue)
			: GameThreadLocalQueue(InGameThreadLocalQueue)
			, WorkerLocalQueue(InWorkerLocalQueue)
		{
		}

		FTask* Dequeue(bool bPermitBackgroundWork)
		{
			const int32 MaxPriority = bPermitBackgroundWork ? int32(ETaskPriority::Count) : int32(ETaskPriority::ForegroundCount);

			for (int32 PriorityIndex = 0; PriorityIndex < MaxPriority; ++PriorityIndex)
			{
				FTask* Item = GameThreadLocalQueue->StealLocal(PriorityIndex);
				if (Item)
				{
					return Item;
				}

				Item = WorkerLocalQueue->Dequeue(PriorityIndex);
				if (Item)
				{
					return Item;
				}
			}
			return nullptr;
		}
	};

	void FScheduler::StandbyLoop(Private::FWaitEvent* WorkerEvent, FSchedulerTls::FLocalQueueType* WorkerLocalQueue, uint32 WaitCycles, bool bPermitBackgroundWork)
	{
		bool bPreparingStandby = false;
		Private::FOutOfWork OutOfWork;

		using FCombinedQueue = TCombinedQueue<FSchedulerTls::FLocalQueueType>;
		FCombinedQueue CombinedQueue(GameThreadLocalQueue.Get(), WorkerLocalQueue);

		while (true)
		{
			bool bExecutedSomething = false;
			while (TryExecuteTaskFrom<FCombinedQueue, &FCombinedQueue::Dequeue, true>(WorkerEvent, &CombinedQueue, OutOfWork, bPermitBackgroundWork)
				|| TryExecuteTaskFrom<FSchedulerTls::FLocalQueueType, &FSchedulerTls::FLocalQueueType::DequeueSteal, true>(WorkerEvent, WorkerLocalQueue, OutOfWork, bPermitBackgroundWork))
			{
				bPreparingStandby = false;
				bExecutedSomething = true;

				// If we're currently oversubscribed... we might be selected for standby even when there is work left.
				WaitingQueue[bPermitBackgroundWork].ConditionalStandby(WorkerEvent);
			}

			// Check if we're shutting down
			if (ActiveWorkers.load(std::memory_order_relaxed) == 0)
			{
				OutOfWork.Stop();
				break;
			}

			if (bExecutedSomething == false)
			{
				if (!bPreparingStandby)
				{
					OutOfWork.Start();
					WaitingQueue[bPermitBackgroundWork].PrepareStandby(WorkerEvent);
					bPreparingStandby = true;
				}
				else if (WaitingQueue[bPermitBackgroundWork].CommitStandby(WorkerEvent, OutOfWork))
				{
					// Only reset this when the commit succeeded, otherwise we're backing off the commit and looking at the queue again
					bPreparingStandby = false;
				}
			}
		}
	}

	void FScheduler::WorkerLoop(Private::FWaitEvent* WorkerEvent, FSchedulerTls::FLocalQueueType* WorkerLocalQueue, uint32 WaitCycles, bool bPermitBackgroundWork)
	{
		using FCombinedQueue = TCombinedQueue<FSchedulerTls::FLocalQueueType>;
		FCombinedQueue CombinedQueue(GameThreadLocalQueue.Get(), WorkerLocalQueue);

		bool bPreparingWait = false;
		Private::FOutOfWork OutOfWork;
		while (true)
		{
			bool bExecutedSomething = false;
			while (TryExecuteTaskFrom<FCombinedQueue, &FCombinedQueue::Dequeue, false>(WorkerEvent, &CombinedQueue, OutOfWork, bPermitBackgroundWork)
				|| TryExecuteTaskFrom<FSchedulerTls::FLocalQueueType, &FSchedulerTls::FLocalQueueType::DequeueSteal, false>(WorkerEvent, WorkerLocalQueue, OutOfWork, bPermitBackgroundWork))
			{
				bPreparingWait = false;
				bExecutedSomething = true;
			}

			// Check if we're shutting down
			if (ActiveWorkers.load(std::memory_order_relaxed) == 0)
			{
				// Don't leave the waiting queue in a bad state
				if (OutOfWork.Stop())
				{
					WaitingQueue[bPermitBackgroundWork].CancelWait(WorkerEvent);
				}
				break;
			}

			if (bExecutedSomething == false)
			{
				if (!bPreparingWait)
				{
					OutOfWork.Start();
					WaitingQueue[bPermitBackgroundWork].PrepareWait(WorkerEvent);
					bPreparingWait = true;
				}
				else if (WaitingQueue[bPermitBackgroundWork].CommitWait(WorkerEvent, OutOfWork, WorkerSpinCycles, WaitCycles))
				{
					// Only reset this when the commit succeeded, otherwise we're backing off the commit and looking at the queue again
					bPreparingWait = false;
				}
			}
		}
	}

	void FScheduler::WorkerMain(Private::FWaitEvent* WorkerEvent, FSchedulerTls::FLocalQueueType* WorkerLocalQueue, uint32 WaitCycles, bool bPermitBackgroundWork)
	{
		FTlsValues& LocalTlsValues = GetTlsValuesRef();

		checkSlow(LocalTlsValues.LocalQueue == nullptr);
		checkSlow(WorkerLocalQueue != nullptr);
		checkSlow(WorkerEvent != nullptr);

		FTaskTagScope WorkerScope(ETaskTag::EWorkerThread);
		LocalTlsValues.ActiveScheduler = this;

		FMemory::SetupTLSCachesOnCurrentThread();
		LocalTlsValues.WorkerType = bPermitBackgroundWork ? FSchedulerTls::EWorkerType::Background : FSchedulerTls::EWorkerType::Foreground;
		LocalTlsValues.SetStandbyWorker(WorkerEvent->bIsStandby);
		LocalTlsValues.LocalQueue = WorkerLocalQueue;

		{
			Private::FOversubscriptionAllowedScope _(true);

			if (WorkerEvent->bIsStandby)
			{
				StandbyLoop(WorkerEvent, WorkerLocalQueue, WaitCycles, bPermitBackgroundWork);
			}
			else
			{
				WorkerLoop(WorkerEvent, WorkerLocalQueue, WaitCycles, bPermitBackgroundWork);
			}
		}

		LocalTlsValues.LocalQueue = nullptr;
		LocalTlsValues.ActiveScheduler = nullptr;
		LocalTlsValues.SetStandbyWorker(false);
		LocalTlsValues.WorkerType = FSchedulerTls::EWorkerType::None;
		FMemory::ClearAndDisableTLSCachesOnCurrentThread();
	}
}