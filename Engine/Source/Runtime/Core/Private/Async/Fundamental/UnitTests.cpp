// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "Misc/AutomationTest.h"

#include "Async/Fundamental/Scheduler.h"
#include "Async/ManualResetEvent.h"
#include "Experimental/Async/AwaitableTask.h"

#include <atomic>

#if WITH_DEV_AUTOMATION_TESTS

namespace Tasks2Tests
{
	using namespace LowLevelTasks;

	// There is no event that can be waited on for low level tasks so
	// this is acceptable for testing purpose only to spin until completion.
	inline void WaitForTask(FTask& Task)
	{
		// Task has already started, wait
		while (!Task.IsCompleted())
		{
			FPlatformProcess::YieldThread();
		}
	}

	inline void WaitForTasks(TArrayView<FTask> Tasks)
	{
		for (FTask& Task : Tasks)
		{
			WaitForTask(Task);
		}
	}

	// For highly recursive workload that waits, we need the waiters
	// to contribute to the work otherwise we'll exhaust all threads
	// and deadlock. So try to expedite to join the work or wait
	// if the work is already started. Only operates on TSharedPtr
	// since we're not waiting for task completion during expedite
	// and may exit scope before the task is scheduled.
	inline void ExpediteOrWaitForTask(TSharedPtr<FTask>& Task)
	{
		if (Task->TryExpedite())
		{
			return;
		}

		// Task has already started, wait
		WaitForTask(*Task);
	}

	inline void ExpediteOrWaitForTasks(TArrayView<TSharedPtr<FTask>> Tasks)
	{
		for (TSharedPtr<FTask>& Task : Tasks)
		{
			ExpediteOrWaitForTask(Task);
		}
	}


	template<int NumTasks>
	void EmptyPerfTest(bool last)
	{
		const double Alpha = 0.01;

		if(last)
		{
			UE_LOGF(LogTemp, Display, "--------------------------EMPTY Perf Test--------------------------------");
		}

		{
			double StartTime = FPlatformTime::Seconds();

			TArray<FTask> Tasks;
			Tasks.AddDefaulted(NumTasks);

			for (int i = 0; i < NumTasks; ++i)
			{
				Tasks[i].Init(TEXT("Perf Test Basic"), [](){});
			}

			double SpawnedTime = FPlatformTime::Seconds();

			for (int i = 0; i < NumTasks; ++i)
			{
				TryLaunch(Tasks[i]);
			}

			WaitForTasks(Tasks);

			double EndTime = FPlatformTime::Seconds();
			double Total = EndTime - StartTime;
			static double TotalAvg = Total;
			TotalAvg = (Alpha * Total) + (1.0 - Alpha) * TotalAvg;

			double Spawned = SpawnedTime - StartTime;
			static double SpawnedAvg = Spawned;
			SpawnedAvg = (Alpha * Spawned) + (1.0 - Alpha) * SpawnedAvg;

			double End = EndTime - SpawnedTime;
			static double EndAvg = End;
			EndAvg = (Alpha * End) + (1.0 - Alpha) * EndAvg;
			if(last)
			{
				UE_LOGF(LogTemp, Display, "EmptyBusyWaitGlobal Basic Performance:          Spawning %d tasks took %f (%f avg) secs total, %f (%f avg) secs spawning and %f (%f avg) secs dispatching", NumTasks, Total, TotalAvg, Spawned, SpawnedAvg, End, EndAvg);
			}
		}

		{
			using namespace LowLevelTasks;

			TArray<FTask> Tasks;
			Tasks.AddDefaulted(NumTasks);

			double StartTime = FPlatformTime::Seconds();

			for (int i = 0; i < NumTasks; i++)
			{
				Tasks[i].Init(TEXT("Empty Perf Test"), [](){});
			}

			FTask SignalTask;
			SignalTask.Init(TEXT("Empty Signal Task"), [&Tasks]()
			{
				for (int i = 0; i < NumTasks; i++)
				{
					TryLaunch(Tasks[i]);
				}
			});

			double SpawnedTime = FPlatformTime::Seconds();

			TryLaunch(SignalTask);
			WaitForTasks(Tasks);
			WaitForTask(SignalTask);

			double EndTime = FPlatformTime::Seconds();
			double Total = EndTime - StartTime;
			static double TotalAvg = Total;
			TotalAvg = (Alpha * Total) + (1.0 - Alpha) * TotalAvg;

			double Spawned = SpawnedTime - StartTime;
			static double SpawnedAvg = Spawned;
			SpawnedAvg = (Alpha * Spawned) + (1.0 - Alpha) * SpawnedAvg;

			double End = EndTime - SpawnedTime;
			static double EndAvg = End;
			EndAvg = (Alpha * End) + (1.0 - Alpha) * EndAvg;
			if(last)
			{
				UE_LOGF(LogTemp, Display, "EmptyBusyWaitLocal Basic Performance:           Spawning %d tasks took %f (%f avg) secs total, %f (%f avg) secs spawning and %f (%f avg) secs dispatching", NumTasks, Total, TotalAvg, Spawned, SpawnedAvg, End, EndAvg);
			}
		}

		{
			TArray<TAwaitableTask<void>> Tasks;
			Tasks.AddDefaulted(NumTasks);

			double StartTime = FPlatformTime::Seconds();

			TAwaitableTask<void> SignalTask;
			SignalTask.InitAndLaunch(TEXT("Empty Awaitable Signal Task"), [&Tasks]()
			{
				for (int i = 0; i < NumTasks; i++)
				{
					Tasks[i].InitAndLaunch(TEXT("Empty Awaitable Perf Test"), [](){});
				}
			});

			double SpawnedTime = FPlatformTime::Seconds();

			SignalTask.Await();
			for (int i = 0; i < NumTasks; i++)
			{
				Tasks[i].Await();
			}

			double EndTime = FPlatformTime::Seconds();
			double Total = EndTime - StartTime;
			static double TotalAvg = Total;
			TotalAvg = (Alpha * Total) + (1.0 - Alpha) * TotalAvg;

			double Spawned = SpawnedTime - StartTime;
			static double SpawnedAvg = Spawned;
			SpawnedAvg = (Alpha * Spawned) + (1.0 - Alpha) * SpawnedAvg;

			double End = EndTime - SpawnedTime;
			static double EndAvg = End;
			EndAvg = (Alpha * End) + (1.0 - Alpha) * EndAvg;
			if(last)
			{
				UE_LOGF(LogTemp, Display, "EmptyBusyWaitLocal Awaitable Performance:       Spawning %d tasks took %f (%f avg) secs total, %f (%f avg) secs spawning and %f (%f avg) secs dispatching", NumTasks, Total, TotalAvg, Spawned, SpawnedAvg, End, EndAvg);
			}
		}

		if(last)
		{
			UE_LOGF(LogTemp, Display, "--------------------------END EMPTY Perf Test-----------------------------");
		}
	}

	template<int NumTasks>
	void BasicPerfTest(bool last)
	{
		const double Alpha = 0.01;
		if(last)
		{
			UE_LOGF(LogTemp, Display, "--------------------------SIMPLE Perf Test--------------------------------");
		}

		struct Test
		{
			static int64 Fibonacci(int64 N)
			{
				if (N > 2)
				{
					int64 Res1 = Test::Fibonacci(N - 1);
					int64 Res2 = Test::Fibonacci(N - 2);
					return Res1 + Res2;
				}
				return 1;
			}
		};

		static int64 SGarbage = 0;
		static int64 STestVal = 15;

		volatile int64& Garbage = SGarbage;
		volatile int64& TestVal = STestVal;

		{
			using namespace LowLevelTasks;
			TArray<TSharedPtr<FTask>> Tasks;
			Tasks.AddDefaulted(NumTasks);

			double StartTime = FPlatformTime::Seconds();

			for (TSharedPtr<FTask>& Task : Tasks)
			{
				Task = MakeShared<FTask>();
				Task->Init(TEXT("Perf Test"), [&Garbage, &TestVal, Task]()
				{
					Garbage = Test::Fibonacci(TestVal);
				});
			}

			TSharedPtr<FTask> SignalTask = MakeShared<FTask>();
			SignalTask->Init(TEXT("Signal Task"), [&Tasks]()
			{
				for (int i = 0; i < NumTasks; i++)
				{
					TryLaunch(*Tasks[i]);
				}
			});

			double SpawnedTime = FPlatformTime::Seconds();

			TryLaunch(*SignalTask);
			ExpediteOrWaitForTasks(Tasks);
			ExpediteOrWaitForTask(SignalTask);

			double EndTime = FPlatformTime::Seconds();
			double Total = EndTime - StartTime;
			static double TotalAvg = Total;
			TotalAvg = (Alpha * Total) + (1.0 - Alpha) * TotalAvg;

			double Spawned = SpawnedTime - StartTime;
			static double SpawnedAvg = Spawned;
			SpawnedAvg = (Alpha * Spawned) + (1.0 - Alpha) * SpawnedAvg;

			double End = EndTime - SpawnedTime;
			static double EndAvg = End;
			EndAvg = (Alpha * End) + (1.0 - Alpha) * EndAvg;

			if(last)
			{
				UE_LOGF(LogTemp, Display, "FibworkBusyWait Basic Performance:              Spawning %d tasks took %f (%f avg) secs total, %f (%f avg) secs spawning and %f (%f avg) secs dispatching", NumTasks, Total, TotalAvg, Spawned, SpawnedAvg, End, EndAvg);
			}
		}

		{
			TArray<TAwaitableTask<int64>> Tasks;
			Tasks.AddDefaulted(NumTasks);

			double StartTime = FPlatformTime::Seconds();

			TAwaitableTask<void> SignalTask;
			SignalTask.InitAndLaunch(TEXT("Simple Awaitable Signal Task"), [&Tasks, &TestVal]()
			{
				for (int i = 0; i < NumTasks; i++)
				{
					Tasks[i].InitAndLaunch(TEXT("Simple Awaitable Perf Test"), [&TestVal]()
					{
						return Test::Fibonacci(TestVal);
					});
				}
			});

			double SpawnedTime = FPlatformTime::Seconds();

			SignalTask.Await();
			for (int i = 0; i < NumTasks; i++)
			{
				Garbage = Tasks[i].Await();
			}

			double EndTime = FPlatformTime::Seconds();
			double Total = EndTime - StartTime;
			static double TotalAvg = Total;
			TotalAvg = (Alpha * Total) + (1.0 - Alpha) * TotalAvg;

			double Spawned = SpawnedTime - StartTime;
			static double SpawnedAvg = Spawned;
			SpawnedAvg = (Alpha * Spawned) + (1.0 - Alpha) * SpawnedAvg;

			double End = EndTime - SpawnedTime;
			static double EndAvg = End;
			EndAvg = (Alpha * End) + (1.0 - Alpha) * EndAvg;

			if(last)
			{
				UE_LOGF(LogTemp, Display, "FibworkBusyWait Awaitable Performance:          Spawning %d tasks took %f (%f avg) secs total, %f (%f avg) secs spawning and %f (%f avg) secs dispatching", NumTasks, Total, TotalAvg, Spawned, SpawnedAvg, End, EndAvg);
			}
		}

		if(last)
		{
			UE_LOGF(LogTemp, Display, "--------------------------END SIMPLE Perf Test--------------------------------");
		}
		SGarbage++;
	}

	template<int NumTasks>
	void StealingPerfTest(bool last)
	{
		const double Alpha = 0.01;
		static int64 SGarbage = 0;
		static int64 STestVal = 13;
		static int64 STheshold = 11;

		using namespace LowLevelTasks;
		struct Test
		{
			static int64 Fibonacci(int64 N)
			{
				if (N > 2)
				{
					int64 Res1 = Test::Fibonacci(N - 1);
					int64 Res2 = Test::Fibonacci(N - 2);
					return Res1 + Res2;
				}
				return 1;
			}

			static int64 FibonacciTasksBasic(int64 N)
			{
				if (N > STheshold)
				{
					int64 Res1, Res2;

					// autodelete will allow us to expedite the task without regard to lifetime
					TSharedPtr<FTask> Task1 = MakeShared<FTask>();
					TSharedPtr<FTask> Task2 = MakeShared<FTask>();

					Task1->Init(TEXT("Basic Fibonacci Test1"), ETaskPriority::Normal, [&Res1, Task1, N]()
					{
						Res1 = Test::FibonacciTasksBasic(N - 1);
					});
					TryLaunch(*Task1);

					Task2->Init(TEXT("Basic Fibonacci Test2"), ETaskPriority::Normal, [&Res2, Task2, N]()
					{
						Res2 = Test::FibonacciTasksBasic(N - 2);
					});
					TryLaunch(*Task2);

					ExpediteOrWaitForTask(Task2);
					ExpediteOrWaitForTask(Task1);
					return Res1 + Res2;
				}
				return Fibonacci(N);
			}

			static int64 FibonacciTasksAwaitable(int64 N)
			{
				if (N > STheshold)
				{
					TAwaitableTask<int64> Task1, Task2;
					Task1.InitAndLaunch(TEXT("Awaitable Fibonacci Test1"), [N]()
					{
						return Test::FibonacciTasksBasic(N - 1);
					});

					Task2.InitAndLaunch(TEXT("Awaitable Fibonacci Test2"), [N]()
					{
						return Test::FibonacciTasksBasic(N - 2);
					});

					return Task2.Await() + Task1.Await();
				}
				return Fibonacci(N);
			}
		};

		if(last)
		{
			UE_LOGF(LogTemp, Display, "--------------------------STEALING Perf Test--------------------------------");
		}
		volatile int64& Garbage = SGarbage;
		volatile int64& TestVal = STestVal;

		{
			TArray<TSharedPtr<FTask>> Tasks;
			Tasks.AddDefaulted(NumTasks);

			double StartTime = FPlatformTime::Seconds();

			for (TSharedPtr<FTask>& Task : Tasks)
			{
				Task = MakeShared<FTask>();
				Task->Init(TEXT("Perf Test Stealing"), ETaskPriority::Normal, [&Garbage, &TestVal, Task]()
				{
					Garbage = Test::FibonacciTasksBasic(TestVal);
				});
			}

			TSharedPtr<FTask> SignalTask = MakeShared<FTask>();
			SignalTask->Init(TEXT("Signal Task"), [&Tasks]()
			{
				for (int32 Index = 0; Index < NumTasks; Index++)
				{
					TryLaunch(*Tasks[Index]);
				}
			});

			double SpawnedTime = FPlatformTime::Seconds();

			TryLaunch(*SignalTask);
			ExpediteOrWaitForTasks(Tasks);
			ExpediteOrWaitForTask(SignalTask);

			double EndTime = FPlatformTime::Seconds();
			double Total = EndTime - StartTime;
			static double TotalAvg = Total;
			TotalAvg = (Alpha * Total) + (1.0 - Alpha) * TotalAvg;

			double Spawned = SpawnedTime - StartTime;
			static double SpawnedAvg = Spawned;
			SpawnedAvg = (Alpha * Spawned) + (1.0 - Alpha) * SpawnedAvg;

			double End = EndTime - SpawnedTime;
			static double EndAvg = End;
			EndAvg = (Alpha * End) + (1.0 - Alpha) * EndAvg;

			if(last)
			{
				UE_LOGF(LogTemp, Display, "FibworkBusyWaitStealing Basic Performance:      Spawning %d tasks took %f (%f avg) secs total, %f (%f avg) secs spawning and %f (%f avg) secs dispatching", NumTasks, Total, TotalAvg, Spawned, SpawnedAvg, End, EndAvg);
			}
		}

		{
			TArray<TAwaitableTask<int64>> Tasks;
			Tasks.AddDefaulted(NumTasks);

			double StartTime = FPlatformTime::Seconds();

			TAwaitableTask<void> SignalTask;
			SignalTask.InitAndLaunch(TEXT("Stealing Awaitable Signal Task"), [&Tasks, &TestVal]()
			{
				for (int i = 0; i < NumTasks; i++)
				{
					Tasks[i].InitAndLaunch(TEXT("Stealing Awaitable Perf Test"), [&TestVal]()
					{
						return Test::FibonacciTasksAwaitable(TestVal);
					});
					//Tasks[i].Await();
				}
			});

			double SpawnedTime = FPlatformTime::Seconds();

			SignalTask.Await();
			for (int i = 0; i < NumTasks; i++)
			{
				Garbage = Tasks[i].Await();
			}

			double EndTime = FPlatformTime::Seconds();
			double Total = EndTime - StartTime;
			static double TotalAvg = Total;
			TotalAvg = (Alpha * Total) + (1.0 - Alpha) * TotalAvg;

			double Spawned = SpawnedTime - StartTime;
			static double SpawnedAvg = Spawned;
			SpawnedAvg = (Alpha * Spawned) + (1.0 - Alpha) * SpawnedAvg;

			double End = EndTime - SpawnedTime;
			static double EndAvg = End;
			EndAvg = (Alpha * End) + (1.0 - Alpha) * EndAvg;

			if(last)
			{
				UE_LOGF(LogTemp, Display, "FibworkBusyWaitStealing Awaitable Performance:  Spawning %d tasks took %f (%f avg) secs total, %f (%f avg) secs spawning and %f (%f avg) secs dispatching", NumTasks, Total, TotalAvg, Spawned, SpawnedAvg, End, EndAvg);
			}
		}

		if(last)
		{
			UE_LOGF(LogTemp, Display, "--------------------------END STEALING Perf Test--------------------------------");
		}
		SGarbage++;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FTasksPerfTests2, "System.Core.LowLevelTasks.PerfTests", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter | EAutomationTestFlags::Disabled);

	bool FTasksPerfTests2::RunTest(const FString& Parameters)
	{
		for(int i = 0; i <= 100; i++)
		{
			EmptyPerfTest<10000>(i == 100);
			BasicPerfTest<10000>(i == 100);
			StealingPerfTest<10000>(i == 100);
		}
		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FTasksLocalGlobalPriorities, "System.Core.LowLevelTasks.LocalGlobalPriorities", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter);
	bool FTasksLocalGlobalPriorities::RunTest(const FString& Parameters)
	{
		using namespace LowLevelTasks;

		// Avoid any kind of task stealing.
		LowLevelTasks::FScheduler::Get().RestartWorkers(0, 1);

		FTask RootTask;
		FTask LocalTask;
		FTask GlobalTask;

		std::atomic_bool GlobalRun{ false };
		std::atomic_bool LocalRun{ false };

		// This test launch 2 tasks
		// One global that is high prio and supposed to run before the local
		// One local that is normal prio which is supposed to run after the global one

		LocalTask.Init(TEXT("LocalTask"), ETaskPriority::BackgroundNormal,
			[&]()
			{
				LocalRun = true;
				verify(GlobalRun == true);
			}
		);

		GlobalTask.Init(TEXT("GlobalTask"), ETaskPriority::BackgroundHigh,
			[&]()
			{
				GlobalRun = true;
				verify(LocalRun == false);
			}
		);

		RootTask.Init(
			TEXT("RootTask"), ETaskPriority::BackgroundNormal,
			[&]()
			{
				// Both task are launched from inside a worker with no additional wake up so that both task should end up being run in the same worker
				verify(TryLaunch(LocalTask, EQueuePreference::LocalQueuePreference, false /*bWakeUpWorker*/));
				verify(TryLaunch(GlobalTask, EQueuePreference::GlobalQueuePreference, false /*bWakeUpWorker*/));
			});

		TryLaunch(RootTask);

		while (!LocalTask.IsCompleted()) {};
		while (!GlobalTask.IsCompleted()) {};
		verify(GlobalRun == true);
		verify(LocalRun == true);

		LowLevelTasks::FScheduler::Get().RestartWorkers();

		return true;
	};

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FTasksUnitTests2, "System.Core.LowLevelTasks.UnitTests", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter);
	bool FTasksUnitTests2::RunTest(const FString& Parameters)
	{
		using namespace LowLevelTasks;

		for (uint32 TestCancel = 0; TestCancel < 2; TestCancel++)
		{
			//Basic Low LevelAPi
			{
				uint32 TestValue = 1337;

				FTask Task;
				Task.Init(TEXT("Basic Test"), [&TestValue]()
				{
					TestValue = 42;
				});
				verify(Task.TryCancel(ECancellationFlags::PrelaunchCancellation));
				verify(Task.TryRevive());
				TryLaunch(Task);

				bool WasCanceled = TestCancel && Task.TryCancel();
				if(WasCanceled)
				{
					WasCanceled = !Task.TryRevive();
				}
				WaitForTask(Task);

				if (WasCanceled)
				{
					verify(TestValue == 1337);
				}
				else
				{
					verify(TestValue == 42);
				}
			}

			//optional Refcounting
			{
				uint32 TestValue = 1337;

				using FTaskHandle = TSharedPtr<FTask, ESPMode::ThreadSafe>;
				FTaskHandle TaskHandle = MakeShared<FTask, ESPMode::ThreadSafe>();

				TaskHandle->Init(TEXT("Refcounted Test"), [&TestValue, TaskHandle]()
				{
					TestValue = 42;
				});
				TryLaunch(*TaskHandle);

				bool WasCanceled = TestCancel && TaskHandle->TryCancel();
				WaitForTask(*TaskHandle);

				if (WasCanceled)
				{
					verify(TestValue == 1337);
				}
				else
				{
					verify(TestValue == 42);
				}
			}

			//expedite
			{
				uint32 TestValue = 1337;

				FTask Task;
				Task.Init(TEXT("expedite Test"), [&TestValue]()
				{
					TestValue = 42;
				});
				verify(!Task.TryExpedite());
				TryLaunch(Task);
				Task.TryExpedite();

				verify(!Task.TryCancel());
				WaitForTask(Task);

				verify(TestValue == 42);
			}
		}

		//example Awaitable Tasks
		{
			uint32 TestValue = 1337;

			TAwaitableTask<void> Task;
			Task.InitAndLaunch(TEXT("Awaitable Test"), [&TestValue]()
			{
				TestValue = 42;
			});

			Task.Await();
			verify(TestValue == 42);
		}

		//optional uniqueptr
		{
			uint32* TestValue = new uint32(1337);

			using FTaskHandle = TUniquePtr<FTask>;
			FTaskHandle TaskHandle = MakeUnique<FTask>();
			FTask& Task = *TaskHandle;
			Task.Init(TEXT("Uniqueptr Test"), [TestValue, TaskHandle = MoveTemp(TaskHandle)]()
			{
				*TestValue = 42;
				delete TestValue;
			});
			TryLaunch(Task);
		}

		//symetric switch
		{
			uint32 TestValue = 1337;

			FTask TaskA;
			FTask TaskB;
			TaskB.Init(TEXT("TaskB"), [&TestValue]()
			{
				TestValue = 42;
			});
			TaskA.Init(TEXT("TaskA"), [&TaskB]() -> FTask*
			{
				return &TaskB;
			});
			verify(TryLaunch(TaskA));
			WaitForTask(TaskB);

			verify(TestValue == 42);
		}
		return true;
	}
}

#endif
