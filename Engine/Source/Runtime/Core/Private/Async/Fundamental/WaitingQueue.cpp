// Copyright (C) 2016 Dmitry Vyukov <dvyukov@google.com>
//
// This Source Code Form is subject to the terms of the Mozilla
// Public License v. 2.0. If a copy of the MPL was not distributed
// with this file, You can obtain one at http://mozilla.org/MPL/2.0/.

// This implementation is based on EventCount.h
// included in the Eigen library but almost everything has been
// rewritten.

#include "Async/Fundamental/WaitingQueue.h"
#include "Async/Fundamental/Scheduler.h"
#include "Async/TaskTrace.h"
#include "Logging/LogMacros.h"
#include "HAL/RunnableThread.h"
#include "Misc/ScopeLock.h"
#include "Misc/ScopeExit.h"
#include "Misc/CommandLine.h"
#include "HAL/PlatformMisc.h"
#include "HAL/PlatformProcess.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "Trace/Trace.h"
#include "Trace/Trace.inl"
#include "Math/Color.h"
#include "Misc/AssertionMacros.h"
#include "CoreGlobals.h"

// Activating the waiting queue tracing can help understand exactly what's going on
// from UnrealInsights or another external profiler.
// 
// Note that we're using empty WAITINGQUEUE_EVENT_SCOPE in almost every condition
// so we can follow along which code path are taken.
#define WITH_WAITINGQUEUE_TRACING 0
#define WITH_WAITINGQUEUE_CHECK   DO_CHECK
#define WITH_WAITINGQUEUE_DEBUG   0

#if WITH_WAITINGQUEUE_TRACING

namespace LowLevelTasks::Impl
{
	// This helps with visibility of events in UnrealInsights during debugging
	// of the waiting queue because any events below the 100 ns resolution
	// can often end up with 0 ns. This makes it very hard to see the order of
	// events since 0 sized events are unzoomable.
	struct NonEmptyEventScope
	{
		uint64 StartCycle{0};
		NonEmptyEventScope()
			: StartCycle(FPlatformTime::Cycles64())
		{
		}
		~NonEmptyEventScope()
		{
			while (StartCycle == FPlatformTime::Cycles64())
			{
			}
		}
	};
}

#define WAITINGQUEUE_EVENT_SCOPE(Name) SCOPED_NAMED_EVENT(Name, FColor::Turquoise) LowLevelTasks::Impl::NonEmptyEventScope __nonEmptyEventScope;
#else
#define WAITINGQUEUE_EVENT_SCOPE(Name)
#endif

#define WAITINGQUEUE_EVENT_SCOPE_ALWAYS(Name) SCOPED_NAMED_EVENT(Name, FColor::Turquoise)

#if WITH_WAITINGQUEUE_DEBUG
UE_DISABLE_OPTIMIZATION
#endif

CSV_DECLARE_CATEGORY_EXTERN(Scheduler);

namespace LowLevelTasks::Private
{
	namespace WaitingQueueImpl
	{
		// State_ layout:
		// - low kWaiterBits is a stack of waiters committed wait
		//   (indexes in NodesArray are used as stack elements,
		//   kStackMask means empty stack).
		// - next kWaiterBits is count of waiters in prewait state.
		// - next kWaiterBits is count of pending signals.
		// - remaining bits are ABA counter for the stack.
		//   (stored in Waiter node and incremented on push).
		static constexpr uint64 WaiterBits = 14;
		static constexpr uint64 StackMask = (1ull << WaiterBits) - 1;
		static constexpr uint64 WaiterShift = WaiterBits;
		static constexpr uint64 WaiterMask = ((1ull << WaiterBits) - 1) << WaiterShift;
		static constexpr uint64 WaiterInc = 1ull << WaiterShift;
		static constexpr uint64 SignalShift = 2 * WaiterBits;
		static constexpr uint64 SignalMask = ((1ull << WaiterBits) - 1) << SignalShift;
		static constexpr uint64 SignalInc = 1ull << SignalShift;
		static constexpr uint64 EpochShift = 3 * WaiterBits;
		static constexpr uint64 EpochBits = 64 - EpochShift;
		static constexpr uint64 EpochMask = ((1ull << EpochBits) - 1) << EpochShift;
		static constexpr uint64 EpochInc = 1ull << EpochShift;

		// Get the active thread count out of the standby state.
		uint64 GetActiveThreadCount(uint64 StandbyState)
		{
			// The StandbyState stores the active thread count in the waiter bits.
			return (StandbyState & WaiterMask) >> WaiterShift;
		}

		void EnterWait(FWaitEvent* Node)
		{
			// Flush any open scope before going to sleep so that anything that happened
			// before appears in UnrealInsights right away. If we don't do this,
			// the thread buffer will be held to this thread until we wake up and fill it
			// so it might cause events to appear as missing in UnrealInsights, especially
			// in case we never wake up again (i.e. deadlock / crash).
			TRACE_CPUPROFILER_EVENT_FLUSH();
			Private::FOversubscriptionAllowedScope _(false /* Disallow oversubscription for this wait */);

			// Let the memory manager know we're inactive so it can do whatever it wants with our
			// thread-local memory cache if we have any.
			FMemory::MarkTLSCachesAsUnusedOnCurrentThread();

			Node->Event->Wait();

			// Let the memory manager know we're active again and need our
			// thread-local memory cache back if we have any.
			FMemory::MarkTLSCachesAsUsedOnCurrentThread();
		}
	}

void FWaitingQueue::Init(uint32 InThreadCount, uint32 InMaxThreadCount, TFunction<void()> InCreateThread, uint32 InActiveThreadCount)
{
	using namespace WaitingQueueImpl;

	ThreadCount = InThreadCount;
	MaxThreadCount = InMaxThreadCount;
	CreateThread = InCreateThread;
	Oversubscription = 0;
	bIsShuttingDown = false;
	State = StackMask;

	// Store the external thread creations in the waiter bits which
	// represent the number of currently active threads.
	StandbyState = StackMask | ((uint64(InActiveThreadCount) << WaiterBits) & WaiterMask);

	check(NodesArray.Num() < (1ull << WaiterBits) - 1);
}

void FWaitingQueue::FinishShutdown()
{
	using namespace WaitingQueueImpl;

	check((State & (StackMask | WaiterMask)) == StackMask);
	check((StandbyState & StackMask) == StackMask);
}

void FWaitingQueue::PrepareWait(FWaitEvent* Node)
{
	using namespace WaitingQueueImpl;

	WAITINGQUEUE_EVENT_SCOPE(FWaitingQueue_PrepareWait);

	State.fetch_add(WaiterInc, std::memory_order_relaxed);
}

bool FWaitingQueue::IsOversubscriptionLimitReached() const
{
	return Oversubscription.load(std::memory_order_relaxed) >= MaxThreadCount;
}

void FWaitingQueue::CheckState(uint64 InState, bool bInIsWaiter)
{
	using namespace WaitingQueueImpl;

	static_assert(EpochBits >= 20, "Not enough bits to prevent ABA problem");
#if WITH_WAITINGQUEUE_CHECK
	const uint64 Waiters = (InState & WaiterMask) >> WaiterShift;
	const uint64 Signals = (InState & SignalMask) >> SignalShift;
	check(Waiters >= Signals);
	check(Waiters < (1 << WaiterBits) - 1);
	check(!bInIsWaiter || Waiters > 0);
	(void)Waiters;
	(void)Signals;
#endif
}

void FWaitingQueue::CheckStandbyState(uint64 InState)
{
	using namespace WaitingQueueImpl;

#if WITH_WAITINGQUEUE_CHECK
	const uint64 Index = (InState & StackMask);
	const uint64 ActiveThreadCount = (InState & WaiterMask) >> WaiterShift;
	const uint64 Signals = (InState & SignalMask) >> SignalShift;
	check(Signals == 0); // Unused in this mode
	check(ActiveThreadCount <= NodesArray.Num());
	check(Index == StackMask || Index < NodesArray.Num())
#endif
}

bool FWaitingQueue::CommitWait(FWaitEvent* Node, FOutOfWork& OutOfWork, int32 SpinCycles, int32 WaitCycles)
{
	using namespace WaitingQueueImpl;

	{
		WAITINGQUEUE_EVENT_SCOPE(FWaitingQueue_CommitWait);

		check((Node->Epoch & ~EpochMask) == 0);
		Node->State.store(EWaitState::NotSignaled, std::memory_order_relaxed);

		uint64 LocalState = State.load(std::memory_order_relaxed);

		CheckState(LocalState, true);
		uint64 NewState;
		if ((LocalState & SignalMask) != 0)
		{
			WAITINGQUEUE_EVENT_SCOPE(CommitWait_TryConsume);
			// Consume the signal and return immediately.
			NewState = LocalState - WaiterInc - SignalInc + EpochInc;
		}
		else
		{
			WAITINGQUEUE_EVENT_SCOPE(CommitWait_TryCommit);
			// Remove this thread from pre-wait counter and add to the waiter stack.
			NewState = ((LocalState & (WaiterMask | EpochMask)) - WaiterInc + EpochInc) | (Node - &NodesArray[0]);
			Node->Next.store(LocalState & StackMask, std::memory_order_relaxed);
		}
		CheckState(NewState);
		if (State.compare_exchange_weak(LocalState, NewState, std::memory_order_acq_rel, std::memory_order_relaxed))
		{
			if ((LocalState & SignalMask) == 0)
			{
				WAITINGQUEUE_EVENT_SCOPE(CommitWait_Success);
				// Fallthrough to park but we want to get out of the CommitWait scope first so it doesn't stick
			}
			else
			{
				WAITINGQUEUE_EVENT_SCOPE(CommitWait_Aborted);
				OutOfWork.Stop();
				return true;
			}
		}
		else
		{
			WAITINGQUEUE_EVENT_SCOPE(CommitWait_Backoff);
			// Avoid too much contention on commit as it's not healthy. 
			// Prefer going back validating if anything has come up in the task queues
			// in between commit retries.
			return false;
		}
	}

	Park(Node, OutOfWork, SpinCycles, WaitCycles);
	return true;
}

bool FWaitingQueue::CancelWait(FWaitEvent* Node)
{
	using namespace WaitingQueueImpl;

	WAITINGQUEUE_EVENT_SCOPE(FWaitingQueue_CancelWait);
	uint64 LocalState = State.load(std::memory_order_relaxed);
	for (;;)
	{
		bool bConsumedSignal = false;
		CheckState(LocalState, true);
		uint64_t NewState = LocalState - WaiterInc;

		// When we consume a signal, the caller will have to try to wake up an additional
		// worker otherwise we could end up missing a wakeup and end up into a deadlock.
		// The more signal we consume, the more spurious wakeups we're going to have so
		// only consume a signal when both waiters and signals are equal so we get the
		// minimal amount of consumed signals possible.
		if (((LocalState & WaiterMask) >> WaiterShift) == ((LocalState & SignalMask) >> SignalShift))
		{
			WAITINGQUEUE_EVENT_SCOPE(Try_ConsumeSignal);
			NewState -= SignalInc;
			bConsumedSignal = true;
		}
		else
		{
			WAITINGQUEUE_EVENT_SCOPE(Try_NoConsumeSignal);
			bConsumedSignal = false;
		}

		CheckState(NewState);
		if (State.compare_exchange_weak(LocalState, NewState, std::memory_order_acq_rel, std::memory_order_relaxed))
		{
			if (bConsumedSignal)
			{
				WAITINGQUEUE_EVENT_SCOPE(Success_SignalConsumed);
				// Since we consumed the event, but we don't know if we're cancelling because of the task
				// this other thread is waking us for or another task entirely. Tell the caller to wake another thread.
				return true;
			}
			else
			{
				WAITINGQUEUE_EVENT_SCOPE(Success_NoSignalConsumed);
			}
			return false;
		}
	}
}

void FWaitingQueue::StartShutdown()
{
	using namespace WaitingQueueImpl;

	bIsShuttingDown = true;

	// Wake up all workers.
	NotifyInternal(NodesArray.Num());

	// Notification above doesn't trigger standby threads
	// during shutdown so trigger them here.
	uint64 LocalState = StandbyState;
	while ((LocalState & StackMask) != StackMask)
	{
		FWaitEvent* Node = &NodesArray[LocalState & StackMask];
		Node->Event->Trigger();
		LocalState = Node->Next;
	}
	StandbyState = StackMask;
}

void FWaitingQueue::PrepareStandby(FWaitEvent* Node)
{
	// We store the whole state before going back checking the queue so that we can't possibly
	// miss an event in-between PrepareStandby and CommitStandby.

	WAITINGQUEUE_EVENT_SCOPE(FWaitingQueue_PrepareStandby);

	Node->Epoch = StandbyState;
}

void FWaitingQueue::ConditionalStandby(FWaitEvent* Node)
{
	using namespace WaitingQueueImpl;

	WAITINGQUEUE_EVENT_SCOPE(FWaitingQueue_ConditionalStandby);

	if (bIsShuttingDown.load(std::memory_order_relaxed))
	{
		return;
	}

	uint64 LocalState = StandbyState;
	while (GetActiveThreadCount(LocalState) > ThreadCount + Oversubscription.load(std::memory_order_relaxed))
	{
		WAITINGQUEUE_EVENT_SCOPE(FWaitingQueue_ConditionalStandby_Iteration);

		CheckStandbyState(LocalState);
		// We store the active thread count in the waiters slot, so decrement it by 1.
		const uint64 Waiters  = (LocalState & WaiterMask) - WaiterInc;
		const uint64 NewEpoch = (LocalState & EpochMask) + EpochInc;
		const uint64 NewState = (Node - &NodesArray[0]) | NewEpoch | Waiters;

		Node->Next.store(LocalState & StackMask);
		Node->Event->Reset();

		CheckStandbyState(NewState);
		if (StandbyState.compare_exchange_weak(LocalState, NewState))
		{
			// After pushing ourselves onto the standby stack, recheck shutdown.
			// StartShutdown drains the stack then does a plain StandbyState = StackMask.
			// If we pushed after the drain but before that store, nobody will trigger
			// our event and StopWorkers will deadlock on Thread->Join().
			if (bIsShuttingDown.load(std::memory_order_acquire))
			{
				Node->Event->Trigger();
				return;
			}
			WAITINGQUEUE_EVENT_SCOPE(Standby);
			EnterWait(Node);
		}
		else
		{
			WAITINGQUEUE_EVENT_SCOPE(Standby_Fail);
		}
	}
}

bool FWaitingQueue::CommitStandby(FWaitEvent* Node, FOutOfWork& OutOfWork)
{
	using namespace WaitingQueueImpl;

	{
		WAITINGQUEUE_EVENT_SCOPE(FWaitingQueue_CommitStandby);

		uint64 LocalState = Node->Epoch;
		CheckStandbyState(LocalState);
		// We store the active thread count in the waiters slot, so decrement it by 1.
		const uint64 Waiters = (LocalState & WaiterMask) - WaiterInc;
		const uint64 Epoch = (LocalState & EpochMask) + EpochInc;
		const uint64 NewState = (Node - &NodesArray[0]) | Epoch | Waiters;

		Node->Next.store(LocalState & StackMask);
		Node->Event->Reset();

		CheckStandbyState(NewState);
		if (StandbyState.compare_exchange_strong(LocalState, NewState))
		{
			// Same shutdown race as ConditionalStandby — recheck after pushing onto the stack.
			if (bIsShuttingDown.load(std::memory_order_acquire))
			{
				Node->Event->Trigger();
				OutOfWork.Stop();
				return true;
			}
			// fallthrough to the end of the function where we wait
		}
		else
		{
			WAITINGQUEUE_EVENT_SCOPE(CommitStandby_Abort);
			// Update the value before we go back checking if new tasks have been queued.
			Node->Epoch = LocalState;
			return false;
		}
	}

	OutOfWork.Stop();
	EnterWait(Node);
	return true;
}

void FWaitingQueue::IncrementOversubscription()
{
	using namespace WaitingQueueImpl;

	if (++Oversubscription >= MaxThreadCount)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FWaitingQueue::OversubscriptionLimitReached);
		CSV_CUSTOM_STAT(Scheduler, OversubscriptionLimitReached, 1, ECsvCustomStatOp::Accumulate);
		OversubscriptionLimitReachedEvent.Broadcast();
	}

	// This is important that StandbyState is invalidated after Oversubscription is increased so we
	// can detect stale decisions and reevaluate oversubscription.
	// Notify -> TryStartNewThread takes care of updating StandbyState for us, but only
	// when standby threads are actually needed.

	Notify();
}

void FWaitingQueue::DecrementOversubscription()
{
	--Oversubscription;
}

bool FWaitingQueue::TryStartNewThread()
{
	WAITINGQUEUE_EVENT_SCOPE(FWaitingQueue_TryStartNewThread);
	using namespace WaitingQueueImpl;

	// Invalidate the current state by adding an Epoch right away so compare-exchange for other threads can detect
	// oversubscription has changed which happens in IncrementOversubscription before calling this function.
	//
	// Important to always read the StandbyState before the Oversubscription value so that we capture the current epoch to validate
	// Oversubscription didn't change while we were doing the CAS.
	uint64 LocalState = StandbyState.fetch_add(EpochInc, std::memory_order_seq_cst) + EpochInc;
	while (GetActiveThreadCount(LocalState) < MaxThreadCount && GetActiveThreadCount(LocalState) < ThreadCount + Oversubscription.load(std::memory_order_relaxed))
	{
		WAITINGQUEUE_EVENT_SCOPE(FWaitingQueue_TryStartNewThread_Iteration);

		CheckStandbyState(LocalState);

		// We store the active thread count in the waiters slot, so increment it by 1.
		const uint64 NewEpoch = (LocalState & EpochMask) + EpochInc;
		uint64 NewState = NewEpoch | (LocalState & WaiterMask) + WaiterInc;
		if ((LocalState & StackMask) != StackMask)
		{
			WAITINGQUEUE_EVENT_SCOPE(FWaitingQueue_TryStartNewThread_FoundNode);
			FWaitEvent* Node = &NodesArray[LocalState & StackMask];
			uint64 Next = Node->Next.load(std::memory_order_relaxed);
			NewState |= Next & StackMask;
		}
		else
		{
			WAITINGQUEUE_EVENT_SCOPE(FWaitingQueue_TryStartNewThread_Empty);
			NewState |= LocalState & StackMask;
		}

		CheckStandbyState(NewState);
		if (StandbyState.compare_exchange_weak(LocalState, NewState, std::memory_order_acq_rel, std::memory_order_relaxed))
		{
			if ((LocalState & StackMask) != StackMask)
			{
				// We got an existing node, wake it from standby
				WAITINGQUEUE_EVENT_SCOPE_ALWAYS(FWaitingQueue_SignalStandbyThread);
				CSV_SCOPED_TIMING_STAT(Scheduler, SignalStandbyThread);
				FWaitEvent* Node = &NodesArray[LocalState & StackMask];
				Node->Event->Trigger();
				return true;
			}
			else if (bIsShuttingDown.load(std::memory_order_relaxed) == false)
			{
				CSV_SCOPED_TIMING_STAT(Scheduler, CreateThread);
				WAITINGQUEUE_EVENT_SCOPE_ALWAYS(FWaitingQueue_CreateThread);
				CreateThread();
				return true;
			}
			else
			{
				WAITINGQUEUE_EVENT_SCOPE(FWaitingQueue_TryStartNewThread_Backoff);
				StandbyState -= WaiterInc;
				return false;
			}
		}
	}

	return false;
}

int32 FWaitingQueue::NotifyInternal(int32 Count)
{
	using namespace WaitingQueueImpl;

	WAITINGQUEUE_EVENT_SCOPE(FWaitingQueue_Notify);

	int32 Notifications = 0;
	while (Count > Notifications)
	{
		uint64 LocalState = State.load(std::memory_order_relaxed);
		for (;;)
		{
			CheckState(LocalState);
			const uint64 Waiters  = (LocalState & WaiterMask) >> WaiterShift;
			const uint64 Signals  = (LocalState & SignalMask) >> SignalShift;
			const uint64 NewEpoch = (LocalState & EpochMask) + EpochInc;
			const bool bNotifyAll = Count >= NodesArray.Num();

			uint64 NewState;
			if ((LocalState & StackMask) == StackMask && Waiters == Signals)
			{
				// No more waiters, go through the CAS to provide proper ordering
				// with other threads entering PrepareWait.
				WAITINGQUEUE_EVENT_SCOPE(TryNoMoreWaiter);
				NewState = LocalState + EpochInc;
			}
			else if (bNotifyAll)
			{
				WAITINGQUEUE_EVENT_SCOPE(TryUnblockAll);
				// Empty wait stack and set signal to number of pre-wait threads.
				NewState = (LocalState & WaiterMask) | (Waiters << SignalShift) | StackMask | NewEpoch;
			}
			else if (Signals < Waiters)
			{
				WAITINGQUEUE_EVENT_SCOPE(TryAbortOnePreWait);
				// There is a thread in pre-wait state, unblock it.
				NewState = LocalState + SignalInc + EpochInc;
			}
			else
			{
				WAITINGQUEUE_EVENT_SCOPE(TryUnparkOne);
				// Pop a waiter from list and unpark it.
				FWaitEvent* Node = &NodesArray[LocalState & StackMask];
				uint64 Next = Node->Next.load(std::memory_order_relaxed);
				NewState = (LocalState & (WaiterMask | SignalMask)) | (Next & StackMask) | NewEpoch;
			}
			CheckState(NewState);
			if (State.compare_exchange_weak(LocalState, NewState, std::memory_order_acq_rel, std::memory_order_relaxed))
			{
				if (!bNotifyAll && (Signals < Waiters))
				{
					WAITINGQUEUE_EVENT_SCOPE(UnblockedPreWaitThread);
					Notifications++;
					break;  // unblocked pre-wait thread
				}

				if ((LocalState & StackMask) == StackMask)
				{
					WAITINGQUEUE_EVENT_SCOPE(NoMoreWaiter);
					if (TryStartNewThread())
					{
						Notifications++;
						break;
					}
					return Notifications;
				}

				FWaitEvent* Node = &NodesArray[LocalState & StackMask];
				if (!bNotifyAll)
				{
					WAITINGQUEUE_EVENT_SCOPE(UnparkOne);
					Node->Next.store(StackMask, std::memory_order_relaxed);
					Notifications += Unpark(Node);
					break;
				}
				else
				{
					WAITINGQUEUE_EVENT_SCOPE(UnparkAll);
					Notifications += (int32)Waiters;
					return Unpark(Node) + Notifications;
				}
			}
		}
	}

	return Notifications;
}

void FWaitingQueue::Park(FWaitEvent* Node, FOutOfWork& OutOfWork, int32 SpinCycles, int32 WaitCycles)
{
	using namespace WaitingQueueImpl;

	{
		ON_SCOPE_EXIT { OutOfWork.Stop(); };
		WAITINGQUEUE_EVENT_SCOPE(FWaitingQueue_Park);

		{
			// Spinning for a very short while helps reduce signaling cost
			// since we're giving the other threads a final chance to wake us with an 
			// atomic only instead of a more costly kernel call.
			WAITINGQUEUE_EVENT_SCOPE(FWaitingQueue_Park_Spin);
			for (int Spin = 0; Spin < SpinCycles; ++Spin)
			{
				if (Node->State.load(std::memory_order_relaxed) == EWaitState::NotSignaled)
				{
					FPlatformProcess::YieldCycles(WaitCycles);
				}
				else
				{
					WAITINGQUEUE_EVENT_SCOPE(FWaitingQueue_Park_Abort);
					return;
				}
			}
		}

		Node->Event->Reset();
		EWaitState Target = EWaitState::NotSignaled;
		if (Node->State.compare_exchange_strong(Target, EWaitState::Waiting, std::memory_order_relaxed, std::memory_order_relaxed))
		{
			WAITINGQUEUE_EVENT_SCOPE(FWaitingQueue_Park_Wait);
			// Fall through to the wait function so we close all inner scope before waiting.
		}
		else
		{
			WAITINGQUEUE_EVENT_SCOPE(FWaitingQueue_Park_Abort);
			return;
		}
	}

	EnterWait(Node);
}

int32 FWaitingQueue::Unpark(FWaitEvent* Node)
{
	using namespace WaitingQueueImpl;

	WAITINGQUEUE_EVENT_SCOPE(FWaitingQueue_Unpark);

	int32 UnparkedCount = 0;
	for (FWaitEvent* Next; Node; Node = Next)
	{
		uint64 NextNode = Node->Next.load(std::memory_order_relaxed) & StackMask;
		Next = NextNode == StackMask ? nullptr : &NodesArray[(int)NextNode];

		UnparkedCount++;

		// Signaling can be very costly on some platforms. So only trigger
		// the event if the other thread was in the waiting state.
		if (Node->State.exchange(EWaitState::Signaled, std::memory_order_relaxed) == EWaitState::Waiting)
		{
			// Always trace this one since signaling cost can be very expensive.
			WAITINGQUEUE_EVENT_SCOPE_ALWAYS(FWaitingQueue_Unpark_SignalWaitingThread);
			Node->Event->Trigger();
		}
		else
		{
			WAITINGQUEUE_EVENT_SCOPE(FWaitingQueue_Unpark_SignaledSpinningThread);
		}
	}

	return UnparkedCount;
}

} // namespace LowLevelTasks::Private

#if WITH_WAITINGQUEUE_DEBUG
UE_ENABLE_OPTIMIZATION
#endif