// Copyright Epic Games, Inc. All Rights Reserved.

#include "Async/InheritedContext.h"
#include "HAL/UnrealMemory.h"

#include <atomic>

namespace UE
{

// ---- Thread-local state for extension mechanism ----

// Head of the per-thread linked list of active extension scopes.
// Pushed/popped by FInheritedContextExtensionScope ctor/dtor.
static thread_local FInheritedContextExtensionScope* GExtensionScopeHead = nullptr;

// Extension data being propagated through a task chain.
// Set by FInheritedContextScope ctor when restoring a captured context that has extensions,
// so that child tasks launched during execution can capture the same extensions.
static thread_local FInheritedContextExtensionData* GPropagatedExtensionData = nullptr;

// ---- FInheritedContextExtensionData ----
// Single-allocation storage: [Header][Entry array][Aligned data blob]

struct FInheritedContextExtensionData
{
	struct FEntry
	{
		FInheritedContextExtension Ext;
		uint32 Offset; // Byte offset within the data blob for this entry's data
	};

#if DO_CHECK
	std::atomic<int32>  RefCount;
#endif
	int32  NumEntries;
	uint32 DataBlockOffset; // Byte offset from `this` to the start of the data blob
	uint32 TotalAllocSize;
	uint32 AllocAlignment;

	FEntry*       GetEntries()       { return reinterpret_cast<FEntry*>(this + 1); }
	const FEntry* GetEntries() const { return reinterpret_cast<const FEntry*>(this + 1); }

	void* GetData(int32 Index)
	{
		return reinterpret_cast<uint8*>(this) + DataBlockOffset + GetEntries()[Index].Offset;
	}

	const void* GetData(int32 Index) const
	{
		return reinterpret_cast<const uint8*>(this) + DataBlockOffset + GetEntries()[Index].Offset;
	}

	// Allocate and populate entries from a list of extension descriptors.
	// Data values are NOT initialized; caller must invoke Capture on each entry.
	static FInheritedContextExtensionData* Allocate(const FInheritedContextExtension* const* Extensions, int32 Num)
	{
		check(Num > 0);

		// Calculate per-entry data offsets with alignment
		uint32 DataSize = 0;
		uint32 MaxDataAlign = 1;
		TArray<uint32, TInlineAllocator<8>> Offsets;
		Offsets.SetNumUninitialized(Num);

		for (int32 i = 0; i < Num; ++i)
		{
			DataSize = Align(DataSize, Extensions[i]->Impl->DataAlign);
			Offsets[i] = DataSize;
			DataSize += Extensions[i]->Impl->DataSize;
			MaxDataAlign = FMath::Max(MaxDataAlign, Extensions[i]->Impl->DataAlign);
		}

		// Layout: [FInheritedContextExtensionData][FEntry * Num][Padding][Data blob]
		const uint32 HeaderAndEntries = static_cast<uint32>(sizeof(FInheritedContextExtensionData) + Num * sizeof(FEntry));
		const uint32 BlockOffset = Align(HeaderAndEntries, MaxDataAlign);
		const uint32 TotalSize = BlockOffset + DataSize;
		const uint32 Alignment = FMath::Max(static_cast<uint32>(alignof(FInheritedContextExtensionData)), MaxDataAlign);

		void* Memory = FMemory::Malloc(TotalSize, Alignment);
		FInheritedContextExtensionData* Data = static_cast<FInheritedContextExtensionData*>(Memory);
#if DO_CHECK
		Data->RefCount.store(0, std::memory_order_relaxed);
#endif
		Data->NumEntries      = Num;
		Data->DataBlockOffset = BlockOffset;
		Data->TotalAllocSize  = TotalSize;
		Data->AllocAlignment  = Alignment;

		FEntry* Entries = Data->GetEntries();
		for (int32 i = 0; i < Num; ++i)
		{
			new (&Entries[i]) FEntry{*Extensions[i], Offsets[i]};
		}

		return Data;
	}

	// Allocate a block with the same layout as Source (for saved state storage).
	// Copies header and entry descriptors but NOT data values.
	static FInheritedContextExtensionData* AllocateLike(const FInheritedContextExtensionData* Source)
	{
		void* Memory = FMemory::Malloc(Source->TotalAllocSize, Source->AllocAlignment);
		FInheritedContextExtensionData* Data = static_cast<FInheritedContextExtensionData*>(Memory);

		// Copy header scalars
#if DO_CHECK
		Data->RefCount.store(0, std::memory_order_relaxed);
#endif
		Data->NumEntries      = Source->NumEntries;
		Data->DataBlockOffset = Source->DataBlockOffset;
		Data->TotalAllocSize  = Source->TotalAllocSize;
		Data->AllocAlignment  = Source->AllocAlignment;

		// Copy-construct entries (bumps TSharedPtr ref counts)
		const FEntry* SrcEntries = Source->GetEntries();
		FEntry* DstEntries = Data->GetEntries();
		for (int32 i = 0; i < Data->NumEntries; ++i)
		{
			new (&DstEntries[i]) FEntry(SrcEntries[i]);
		}

		return Data;
	}

	// Call Destroy on each entry's data and free the allocation.
	void DestroyAndFree()
	{
#if DO_CHECK
		check(RefCount.load(std::memory_order_relaxed) == 0);
#endif
		FEntry* Entries = GetEntries();
		for (int32 i = 0; i < NumEntries; ++i)
		{
			if (Entries[i].Ext.Impl->Destroy)
			{
				Entries[i].Ext.Impl->Destroy(GetData(i), Entries[i].Ext.Impl->UserData);
			}
			Entries[i].~FEntry();
		}
		FMemory::Free(this);
	}
};

// ---- FInheritedContextExtensionScope ----

FInheritedContextExtensionScope::FInheritedContextExtensionScope(const FInheritedContextExtension& InExtension)
	: Extension(InExtension)
	, Next(GExtensionScopeHead)
{
	GExtensionScopeHead = this;
}

FInheritedContextExtensionScope::~FInheritedContextExtensionScope()
{
	checkf(GExtensionScopeHead == this, TEXT("FInheritedContextExtensionScope destroyed out of LIFO order"));
	GExtensionScopeHead = Next;
}

// ---- FInheritedContextScope ----

FInheritedContextScope::FInheritedContextScope(TSharedPtr<const FAppTime> InAppTime
	#if ENABLE_LOW_LEVEL_MEM_TRACKER
		, const FLLMActiveTagsCapture& InInheritedLLMTag
	#endif
	#if (UE_MEMORY_TAGS_TRACE_ENABLED && UE_TRACE_ENABLED)
		, int32 InInheritedMemTag
	#endif
	#if (UE_TRACE_METADATA_ENABLED && UE_TRACE_ENABLED)
		, uint32 InInheritedMetadataId
	#endif
		, FInheritedContextExtensionData* InExtensionData
	)
	: PrevAppTime(FAppTime::Fork())
	#if ENABLE_LOW_LEVEL_MEM_TRACKER
		, LLMScopes(InInheritedLLMTag)
	#endif
	#if (UE_MEMORY_TAGS_TRACE_ENABLED && UE_TRACE_ENABLED)
		, MemScope(InInheritedMemTag)
	#endif
	#if (UE_TRACE_METADATA_ENABLED && UE_TRACE_ENABLED)
		, MetaScope(InInheritedMetadataId)
	#endif
{
	FAppTime::Restore(MoveTemp(InAppTime));

	// Save and update propagated extension data
	PrevPropagatedData = GPropagatedExtensionData;

	if (InExtensionData && InExtensionData->NumEntries > 0)
	{
		// Allocate saved state block with the same layout
		ExtensionSavedState = FInheritedContextExtensionData::AllocateLike(InExtensionData);

		// Apply each extension in reverse order: entries are stored in LIFO order
		// (most recent scope first), so iterating backward replays them in definition
		// order — the innermost (last defined) scope is applied last and its value wins,
		// matching what the parent thread sees.
		const FInheritedContextExtensionData::FEntry* Entries = InExtensionData->GetEntries();
		for (int32 i = InExtensionData->NumEntries - 1; i >= 0; --i)
		{
			Entries[i].Ext.Impl->Apply(
				InExtensionData->GetData(i),
				ExtensionSavedState->GetData(i),
				Entries[i].Ext.Impl->UserData
			);
		}
	}

#if DO_CHECK
	if (InExtensionData)
	{
		InExtensionData->RefCount.fetch_add(1, std::memory_order_relaxed);
	}
#endif

	// Set propagated data so child tasks launched during this scope capture the same extensions.
	// Safe: InExtensionData is owned by the FInheritedContextBase which outlives this scope.
	GPropagatedExtensionData = InExtensionData;
}

FInheritedContextScope::~FInheritedContextScope()
{
	// Restore extensions in forward order: since entries are in LIFO order and
	// Apply iterated backward (definition order), restoring forward unwinds in
	// reverse definition order — last applied is first restored (standard RAII).
	if (ExtensionSavedState)
	{
		const FInheritedContextExtensionData::FEntry* Entries = ExtensionSavedState->GetEntries();
		for (int32 i = 0; i < ExtensionSavedState->NumEntries; ++i)
		{
			Entries[i].Ext.Impl->Restore(
				ExtensionSavedState->GetData(i),
				Entries[i].Ext.Impl->UserData
			);
		}

		ExtensionSavedState->DestroyAndFree();
	}

#if DO_CHECK
	if (GPropagatedExtensionData)
	{
		check(GPropagatedExtensionData->RefCount.load(std::memory_order_relaxed) > 0);
		GPropagatedExtensionData->RefCount.fetch_sub(1, std::memory_order_relaxed);
	}
#endif

	// Restore previous propagated data
	GPropagatedExtensionData = PrevPropagatedData;

	FAppTime::Restore(MoveTemp(PrevAppTime));
}

// ---- FInheritedContextBase ----

void FInheritedContextBase::CaptureInheritedContext()
{
	AppTime = FAppTime::Fork();

#if ENABLE_LOW_LEVEL_MEM_TRACKER
	InheritedLLMTags.CaptureActiveTagData();
#endif

#if UE_MEMORY_TAGS_TRACE_ENABLED && UE_TRACE_ENABLED
	InheritedMemTag = MemoryTrace_GetActiveTag();
#endif

#if UE_TRACE_METADATA_ENABLED && UE_TRACE_ENABLED
	InheritedMetadataId = UE_TRACE_METADATA_SAVE_STACK();
#endif

	// Free previous extension data if CaptureInheritedContext was already called.
	// This handles reuse of the same FInheritedContextBase across multiple dispatches.
	if (ExtensionData)
	{
		ExtensionData->DestroyAndFree();
		ExtensionData = nullptr;
	}

	// Capture generic extensions from active scope chain and propagated data
	if (GExtensionScopeHead || GPropagatedExtensionData)
	{
		TArray<const FInheritedContextExtension*, TInlineAllocator<8>> Extensions;

		// Collect from active scope chain (most recent first due to LIFO linked list)
		for (FInheritedContextExtensionScope* Scope = GExtensionScopeHead; Scope; Scope = Scope->Next)
		{
			Extensions.Add(&Scope->Extension);
		}

		// Collect from propagated data, deduplicating by Impl pointer identity.
		// The same extension can appear in both the scope chain and propagated data
		// when a task is retracted (executed inline on the launching thread).
		// Expected small N (typically 1-3 extensions); linear scan is sufficient.
		if (GPropagatedExtensionData)
		{
			const FInheritedContextExtensionData::FEntry* PropEntries = GPropagatedExtensionData->GetEntries();
			for (int32 i = 0; i < GPropagatedExtensionData->NumEntries; ++i)
			{
				const FInheritedContextExtension* Ext = &PropEntries[i].Ext;
				bool bDuplicate = false;
				for (const FInheritedContextExtension* E : Extensions)
				{
					if (E->Impl.Get() == Ext->Impl.Get())
					{
						bDuplicate = true;
						break;
					}
				}
				if (!bDuplicate)
				{
					Extensions.Add(Ext);
				}
			}
		}

		if (Extensions.Num() > 0)
		{
			ExtensionData = FInheritedContextExtensionData::Allocate(Extensions.GetData(), Extensions.Num());
			for (int32 i = 0; i < Extensions.Num(); ++i)
			{
				Extensions[i]->Impl->Capture(ExtensionData->GetData(i), Extensions[i]->Impl->UserData);
			}
		}
	}
}

FInheritedContextBase::~FInheritedContextBase()
{
	if (ExtensionData)
	{
		ExtensionData->DestroyAndFree();
	}
}

FInheritedContextBase& FInheritedContextBase::operator=(FInheritedContextBase&& Other)
{
	if (this != &Other)
	{
		if (ExtensionData)
		{
			ExtensionData->DestroyAndFree();
		}
		AppTime = MoveTemp(Other.AppTime);
#if ENABLE_LOW_LEVEL_MEM_TRACKER
		InheritedLLMTags = Other.InheritedLLMTags;
#endif
#if UE_MEMORY_TAGS_TRACE_ENABLED && UE_TRACE_ENABLED
		InheritedMemTag = Other.InheritedMemTag;
#endif
#if UE_TRACE_METADATA_ENABLED && UE_TRACE_ENABLED
		InheritedMetadataId = Other.InheritedMetadataId;
#endif
		ExtensionData = Other.ExtensionData;
		Other.ExtensionData = nullptr;
	}
	return *this;
}

// this method must be defined in a .cpp file to avoid exporting dependency on trace module
[[nodiscard]] FInheritedContextScope FInheritedContextBase::RestoreInheritedContext()
{
	return FInheritedContextScope
	(
		AppTime
	#if ENABLE_LOW_LEVEL_MEM_TRACKER
		, InheritedLLMTags
	#endif
	#if (UE_MEMORY_TAGS_TRACE_ENABLED && UE_TRACE_ENABLED)
		, InheritedMemTag
	#endif
	#if (UE_TRACE_METADATA_ENABLED && UE_TRACE_ENABLED)
		, InheritedMetadataId
	#endif
		, ExtensionData
	);
}

}
