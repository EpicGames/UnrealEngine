// Copyright Epic Games, Inc. All Rights Reserved.

#include "Async/Mutex.h"

#include "Async/IntrusiveMutex.h"

namespace UE
{

struct FMutex::FParams
{
	constexpr static uint8 IsLockedFlag = FMutex::IsLockedFlag;
	constexpr static uint8 MayHaveWaitingLockFlag = FMutex::MayHaveWaitingLockFlag;
};

FORCENOINLINE void FMutex::LockSlow()
{
	TIntrusiveMutex<FParams>::LockLoop(State);
}

FORCENOINLINE void FMutex::WakeWaitingThread()
{
	TIntrusiveMutex<FParams>::WakeWaitingThread(State);
}

} // UE
