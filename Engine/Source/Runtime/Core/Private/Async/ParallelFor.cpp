// Copyright Epic Games, Inc. All Rights Reserved.

#include "Async/ParallelFor.h"

#include "CoreTypes.h"
#include "HAL/IConsoleManager.h"

CORE_API int32 GParallelForBackgroundYieldingTimeoutMs = 8;
static FAutoConsoleVariableRef CVarParallelForBackgroundYieldingTimeout(
	TEXT("Async.ParallelFor.YieldingTimeout"),
	GParallelForBackgroundYieldingTimeoutMs,
	TEXT("The timeout (in ms) when background priority parallel for task will yield execution to give higher priority tasks the chance to run.")
);

CORE_API bool GParallelForDisableOversubscription = false;
static FAutoConsoleVariableRef CVarParallelForDisableOversubscription(
	TEXT("Async.ParallelFor.DisableOversubscription"),
	GParallelForDisableOversubscription,
	TEXT("If true, do not enable new threads to handle tasks while waiting for a ParallelFor to finish, because new threads can decrease overall performance.")
);