// Copyright Epic Games, Inc. All Rights Reserved.

#include "Async/ParkingLot.h"

#include "Algo/Sort.h"
#include "Async/UniqueLock.h"
#include "Async/WordMutex.h"
#include "AutoRTFM.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "HAL/PlatformManualResetEvent.h"
#include "Memory/VirtualLinearAllocator.h"
#include "Templates/RefCounting.h"
#include "Templates/NoDestroy.h"
#include <atomic>

#if !UE_ENABLE_VIRTUAL_LINEAR_ALLOCATOR
#include "HAL/Allocators/AnsiAllocator.h"
#include "HAL/MallocAnsi.h"
#endif

namespace UE::ParkingLot::Private
{

/**
 * Identifies which callback function is currently executing.
 *
 * NOTE: ORDER MATTERS FOR COMPARISONS!
 */
enum class EExecutingFunction : uint8
{
	None = 0,
	BeforeWait = 1,
	CanWait = 2,
	OnWakeState = 3,
};

const TCHAR* LexToString(EExecutingFunction Function)
{
	switch (Function)
	{
	default:
	case EExecutingFunction::None: return TEXT("None");
	case EExecutingFunction::BeforeWait: return TEXT("BeforeWait");
	case EExecutingFunction::CanWait: return TEXT("CanWait");
	case EExecutingFunction::OnWakeState: return TEXT("OnWakeState");
	}
}

static thread_local EExecutingFunction ParkingLotExecutingFunction = EExecutingFunction::None;

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

#if UE_ENABLE_VIRTUAL_LINEAR_ALLOCATOR
/**
 * Allocator for permanent allocations by ParkingLot: FTable, FBucket, FThread.
 *
 * Shared by FTable, TObjectAllocator<FBucket>, TObjectAllocator<FThread>.
 * ParkingLot allocates approximately ThreadCount buckets and threads, at 64 bytes each,
 * and tables totaling approximately 2*ThreadCount 8-byte pointers.
 * Even 16,384 threads will consume only 2.25 MiB, which makes 4 MiB plenty.
 *
 * Adds 64 KiB to avoid a multiple of 2 MiB as it can cause issues on platforms with transparent large pages.
 */
static TNoDestroy<TVirtualLinearAllocator<FWordMutex>> ParkingLotPermanentAllocator(4 * 1024 * 1024 + 64 * 1024);
#else
static struct
{
	UE_REWRITE void* Allocate(SIZE_T Size, uint32 Alignment)
	{
		return AnsiMalloc(Size, Alignment);
	}
} ParkingLotPermanentAllocator;
#endif

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * A low-level allocator that maintains a free list for a given object type. Memory is never released.
 */
template <typename ObjectType>
class TObjectAllocator
{
	static_assert(sizeof(ObjectType) >= sizeof(void*));
	static_assert(alignof(ObjectType) >= alignof(void*));

public:
	static void* Allocate()
	{
		if (TUniqueLock Lock(FreeListMutex); void* Mem = FreeListHead)
		{
			FreeListHead = *(void**)Mem;
			return Mem;
		}
		return ParkingLotPermanentAllocator->Allocate(sizeof(ObjectType), alignof(ObjectType));
	}

	static void Free(void* Mem)
	{
		TUniqueLock Lock(FreeListMutex);
		void* Next = FreeListHead;
		*(void**)Mem = Next;
		FreeListHead = Mem;
	}

private:
	inline static FWordMutex FreeListMutex;
	inline static void* FreeListHead = nullptr;
};

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

#if UE_ENABLE_VIRTUAL_LINEAR_ALLOCATOR
/**
 * Allocator for scratch allocations by ParkingLot: TArray<PointerType>.
 *
 * Scratch allocations are bounded by ThreadCount:
 * 1. FTable::Reserve can allocate maximum 3*ThreadCount 8-byte pointers while locked,
 *    and ThreadCount 8-byte pointers while unlocked. Reserve is only called while starting
 *    a new thread, which limits the number of concurrent calls. Even 256 new threads racing
 *    to increase the size to 1024 will allocate a worst-case ~2 MiB.
 * 2. WakeMultiple can allocate maximum WaitingThreadCount 8-byte pointers across every
 *    thread that is waking other threads. Wakes can race with timeouts to bring the
 *    effective total higher but even 512 threads waking 512 threads is only 2 MiB.
 *
 * Adds 64 KiB to avoid a multiple of 2 MiB as it can cause issues on platforms with transparent large pages.
 */
inline static TNoDestroy<TVirtualLinearAllocator<FWordMutex>> ParkingLotScratchAllocator(4 * 1024 * 1024 + 64 * 1024);

/**
 * A container allocator backed by a virtual linear allocator for scratch allocations.
 */
class FScratchAllocator
{
public:
	using SizeType = int32;
	enum { NeedsElementType = false };
	enum { RequireRangeCheck = false };
	enum { ShrinkByDefault = false };

	class ForAnyElementType
	{
	public:
		constexpr ForAnyElementType() = default;

		inline void MoveToEmpty(ForAnyElementType& Other)
		{
			Data = Other.Data;
			Other.Data = nullptr;
		}

		inline void ResizeAllocation(SizeType CurrentNum, SizeType NewMax, SIZE_T NumBytesPerElement, uint32 AlignmentOfElement)
		{
			if (NewMax)
			{
				FScriptContainerElement* const OldData = Data;
				Data = (FScriptContainerElement*)ParkingLotScratchAllocator->Allocate(NewMax * NumBytesPerElement, AlignmentOfElement);
				if (OldData && CurrentNum > 0)
				{
					FMemory::Memcpy(Data, OldData, CurrentNum * NumBytesPerElement);
				}
			}
			else
			{
				Data = nullptr;
			}
		}

		inline SizeType CalculateSlackReserve(SizeType NewMax, SIZE_T NumBytesPerElement, uint32 AlignmentOfElement) const
		{
			return DefaultCalculateSlackReserve(NewMax, NumBytesPerElement, /*bAllowQuantize*/ false, AlignmentOfElement);
		}

		inline SizeType CalculateSlackShrink(SizeType NewMax, SizeType CurrentMax, SIZE_T NumBytesPerElement, uint32 AlignmentOfElement) const
		{
			return DefaultCalculateSlackShrink(NewMax, CurrentMax, NumBytesPerElement, /*bAllowQuantize*/ false, AlignmentOfElement);
		}

		inline SizeType CalculateSlackGrow(SizeType NewMax, SizeType CurrentMax, SIZE_T NumBytesPerElement, uint32 AlignmentOfElement) const
		{
			return DefaultCalculateSlackGrow(NewMax, CurrentMax, NumBytesPerElement, /*bAllowQuantize*/ false, AlignmentOfElement);
		}

		inline SIZE_T GetAllocatedSize(SizeType CurrentMax, SIZE_T NumBytesPerElement) const
		{
			return CurrentMax * NumBytesPerElement;
		}

		inline FScriptContainerElement* GetAllocation() const { return Data; }
		inline bool HasAllocation() const { return !!Data; }

		constexpr SizeType GetInitialCapacity() const { return 0; }

	private:
		ForAnyElementType(const ForAnyElementType&);
		ForAnyElementType& operator=(const ForAnyElementType&);

		FScriptContainerElement* Data = nullptr;
	};

	template <typename ElementType>
	class ForElementType : public ForAnyElementType
	{
	public:
		inline ElementType* GetAllocation() const
		{
			return (ElementType*)ForAnyElementType::GetAllocation();
		}
	};
};
#else
static struct
{
	constexpr void AddRef() {}
	constexpr void Release() {}
} ParkingLotScratchAllocator;
using FScratchAllocator = FAnsiAllocator;
#endif // UE_ENABLE_VIRTUAL_LINEAR_ALLOCATOR

} // UE::ParkingLot::Private

#if UE_ENABLE_VIRTUAL_LINEAR_ALLOCATOR
template <>
struct TAllocatorTraits<UE::ParkingLot::Private::FScratchAllocator> : TAllocatorTraitsBase<UE::ParkingLot::Private::FScratchAllocator>
{
	enum { IsZeroConstruct = true };
	enum { SupportsElementAlignment = true };
};
#endif

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

namespace UE::ParkingLot::Private
{

/**
 * A thread as stored in the wait queue.
 */
struct alignas(PLATFORM_CACHE_LINE_SIZE) FThread final
{
	FThread* Next = nullptr;
	std::atomic<const void*> WaitAddress = nullptr;
	uint64 WakeToken = 0;
	FPlatformManualResetEvent Event;
	std::atomic<uint32> ReferenceCount = 0;

	inline void AddRef()
	{
		ReferenceCount.fetch_add(1, std::memory_order_relaxed);
	}

	inline void Release()
	{
		if (ReferenceCount.fetch_sub(1, std::memory_order_acq_rel) == 1)
		{
			this->~FThread();
			Allocator.Free(this);
		}
	}

	inline static FThread* New()
	{
		return new(Allocator.Allocate()) FThread;
	}

private:
	FThread() = default;
	~FThread() = default;

	FThread(const FThread&) = delete;
	FThread& operator=(const FThread&) = delete;

	static TObjectAllocator<FThread> Allocator;
};

TObjectAllocator<FThread> FThread::Allocator;

class FThreadLocalData
{
public:
	static TRefCountPtr<FThread> Get()
	{
		static thread_local FThreadLocalData ThreadLocalData;
		FThreadLocalData& LocalThreadLocalData = ThreadLocalData;
		if (LIKELY(LocalThreadLocalData.Thread))
		{
			return LocalThreadLocalData.Thread;
		}
		if (LocalThreadLocalData.bDestroyed)
		{
			return FThread::New();
		}
		LocalThreadLocalData.Thread = FThread::New();
		return LocalThreadLocalData.Thread;
	}

private:
	inline static std::atomic<uint32> ThreadCount = 0;

	TRefCountPtr<FThread> Thread;
	bool bDestroyed = false;

	FThreadLocalData();
	~FThreadLocalData();

	FThreadLocalData(const FThreadLocalData&) = delete;
	FThreadLocalData& operator=(const FThreadLocalData&) = delete;
};

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

enum class EQueueAction
{
	Stop,
	Continue,
	RemoveAndStop,
	RemoveAndContinue,
};

/**
 * A bucket in the hash table keyed by memory address.
 *
 * Buckets must be locked to access the list of waiting threads.
 * Buckets are aligned to a cache line to reduce false sharing.
 */
class alignas(PLATFORM_CACHE_LINE_SIZE) FBucket final
{
public:
	static FBucket* Create()
	{
		return new(Allocator.Allocate()) FBucket;
	}

	static void Destroy(FBucket* Bucket)
	{
		Bucket->~FBucket();
		Allocator.Free(Bucket);
	}

	[[nodiscard]] inline TDynamicUniqueLock<FWordMutex> LockDynamic() { return TDynamicUniqueLock(Mutex); }

	inline void Lock() { Mutex.Lock(); }
	inline void Unlock() { Mutex.Unlock(); }

	inline bool IsEmpty() const { return !Head; }

	void Enqueue(FThread* Thread);
	FThread* Dequeue();

	/**
	 * Dequeues threads based on a visitor.
	 *
	 * Visitor signature is EQueueAction (FThread&).
	 * Visitor is called for every thread in the bucket, from head to tail.
	 * Threads are dequeued if the returned action contains Remove.
	 * Visiting stops if the returned action contains Stop.
	 */
	template <typename VisitorType>
	void DequeueIf(VisitorType&& Visitor);

private:
	FBucket() = default;
	~FBucket() = default;

	FBucket(const FBucket&) = delete;
	FBucket& operator=(const FBucket&) = delete;

	FWordMutex Mutex;
	FThread* Head = nullptr;
	FThread* Tail = nullptr;

	static TObjectAllocator<FBucket> Allocator;
};

TObjectAllocator<FBucket> FBucket::Allocator;

void FBucket::Enqueue(FThread* Thread)
{
	checkSlow(Thread);
	checkSlow(!Thread->Next);
	if (Tail)
	{
		Tail->Next = Thread;
		Tail = Thread;
	}
	else
	{
		Head = Thread;
		Tail = Thread;
	}
}

FThread* FBucket::Dequeue()
{
	FThread* Thread = Head;
	if (Thread)
	{
		Head = Thread->Next;
		Thread->Next = nullptr;
		if (Tail == Thread)
		{
			Tail = nullptr;
		}
	}
	return Thread;
}

template <typename VisitorType>
void FBucket::DequeueIf(VisitorType&& Visitor)
{
	FThread** Next = &Head;
	FThread* Prev = nullptr;
	while (FThread* Thread = *Next)
	{
		switch (Visitor(Thread))
		{
		case EQueueAction::Stop:
			return;
		case EQueueAction::Continue:
			Prev = Thread;
			Next = &Thread->Next;
			break;
		case EQueueAction::RemoveAndStop:
			if (Tail == Thread)
			{
				Tail = Prev;
			}
			*Next = Thread->Next;
			Thread->Next = nullptr;
			return;
		case EQueueAction::RemoveAndContinue:
			if (Tail == Thread)
			{
				Tail = Prev;
			}
			*Next = Thread->Next;
			Thread->Next = nullptr;
			break;
		}
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * A hash table of queues of waiting threads keyed by memory address.
 *
 * Tables are never freed. The size of the table is bounded by the maximum number of threads that
 * exist concurrently and have used the wait queue. The table grows by powers of two, which means
 * that the maximum size leaked is less than the maximum size that the table ever grows to. Table
 * leaks are also limited in size because a table is an array of bucket pointers, and the buckets
 * are reused when the table grows.
 */
class FTable final
{
	/** Minimum bucket count to create a table with. */
	constexpr static uint32 MinSize = 32;

public:
	/** Find or create, and lock, the bucket for the memory address. */
	static FBucket& FindOrCreateBucket(const void* Address, TDynamicUniqueLock<FWordMutex>& OutLock);

	/** Find and lock the bucket for the memory address. Returns null if the bucket was not created. */
	static FBucket* FindBucket(const void* Address, TDynamicUniqueLock<FWordMutex>& OutLock);

	/** Reserve memory for the table to handle at least ThreadCount waiting threads. */
	static void Reserve(uint32 ThreadCount);

private:
	/** Returns the current global table, creating it if it does not exist. */
	static FTable& CreateOrGet();

	/** Create a new table with the specified bucket count. */
	static FTable& Create(uint32 Size);
	/** Destroy a table. Must not be called on any table that has been globally visible. */
	static void Destroy(FTable& Table);

	/** Try to lock the whole table by locking every one of its buckets. */
	static bool TryLock(FTable& Table, TArray<FBucket*, FScratchAllocator>& OutBuckets);
	/** Unlock an array of buckets that was filled by TryLock. */
	static void Unlock(TConstArrayView<FBucket*> LockedBuckets);

	/** Calculate a 32-bit hash from the memory address. */
	static uint32 HashAddress(const void* Address);

	/** Pointer to the current global table. Previous global tables are leaked. */
	inline static std::atomic<FTable*> GlobalTable;
	inline static FWordMutex CreateMutex;

	FTable() = default;
	~FTable() = default;

	FTable(const FTable&) = delete;
	FTable& operator=(const FTable&) = delete;

	/** Find or create the bucket at the index. */
	template <typename AllocatorType>
	FBucket& FindOrCreateBucket(uint32 Index, AllocatorType&& BucketAllocator);

	uint32 BucketCount = 0;
	std::atomic<FBucket*> Buckets[0];
};

FBucket& FTable::FindOrCreateBucket(const void* Address, TDynamicUniqueLock<FWordMutex>& OutLock)
{
	const uint32 Hash = HashAddress(Address);

	for (;;)
	{
		FTable& Table = CreateOrGet();
		const uint32 Index = Hash % Table.BucketCount;
		FBucket& Bucket = Table.FindOrCreateBucket(Index, [] { return FBucket::Create(); });
		OutLock = Bucket.LockDynamic();

		if (LIKELY(&Table == GlobalTable.load(std::memory_order_acquire)))
		{
			return Bucket;
		}

		// Restart because the table was resized since it was loaded above.
		OutLock.Unlock();
	}
}

FBucket* FTable::FindBucket(const void* Address, TDynamicUniqueLock<FWordMutex>& OutLock)
{
	const uint32 Hash = HashAddress(Address);

	for (;;)
	{
		if (FTable* Table = GlobalTable.load(std::memory_order_acquire))
		{
			const uint32 Index = Hash % Table->BucketCount;
			if (FBucket* Bucket = Table->Buckets[Index].load(std::memory_order_acquire))
			{
				OutLock = Bucket->LockDynamic();

				if (LIKELY(Table == GlobalTable.load(std::memory_order_acquire)))
				{
					return Bucket;
				}

				// Restart because the table was resized since it was loaded above.
				OutLock = {};
				continue;
			}
		}
		return nullptr;
	}
}

template <typename AllocatorType>
FBucket& FTable::FindOrCreateBucket(uint32 Index, AllocatorType&& BucketAllocator)
{
	std::atomic<FBucket*>& BucketPtr = Buckets[Index];
	FBucket* Bucket = BucketPtr.load(std::memory_order_acquire);
	if (UNLIKELY(!Bucket))
	{
		FBucket* NewBucket = BucketAllocator();
		if (BucketPtr.compare_exchange_strong(Bucket, NewBucket, std::memory_order_release, std::memory_order_acquire))
		{
			Bucket = NewBucket;
		}
		else
		{
			FBucket::Destroy(NewBucket);
		}
		checkSlow(Bucket);
	}
	return *Bucket;
}

FTable& FTable::CreateOrGet()
{
	if (FTable* Table = GlobalTable.load(std::memory_order_acquire); LIKELY(Table))
	{
		return *Table;
	}

	TUniqueLock Lock(CreateMutex);
	if (FTable* Table = GlobalTable.load(std::memory_order_acquire))
	{
		return *Table;
	}

	FTable& NewTable = Create(MinSize);
	FTable* CompareTable = GlobalTable.exchange(&NewTable, std::memory_order_release);
	checkSlow(!CompareTable);
	return NewTable;
}

FTable& FTable::Create(const uint32 Size)
{
	const SIZE_T MemorySize = sizeof(FTable) + sizeof(std::atomic<FBucket*>) * SIZE_T(Size);
	void* const Memory = ParkingLotPermanentAllocator->Allocate(MemorySize, alignof(FTable));
	FMemory::Memzero(Memory, MemorySize);
	FTable& Table = *new(Memory) FTable;
	Table.BucketCount = Size;
	return Table;
}

void FTable::Destroy(FTable& Table)
{
	Table.~FTable();
}

void FTable::Reserve(const uint32 ThreadCount)
{
	// Reference the allocator to allow it to decommit scratch memory.
	TRefCountPtr ScratchAllocatorRef(&ParkingLotScratchAllocator);

	const uint32 TargetBucketCount = FMath::RoundUpToPowerOfTwo(ThreadCount);
	TArray<FBucket*, FScratchAllocator> ExistingBuckets;

	for (;;)
	{
		FTable& ExistingTable = CreateOrGet();

		if (LIKELY(ExistingTable.BucketCount >= TargetBucketCount))
		{
			// Reserve is called every time a thread is created and has amortized constant time
			// because of its power-of-two table growth. Most calls return here without locking.
			return;
		}

		if (!TryLock(ExistingTable, ExistingBuckets))
		{
			continue;
		}

		// Gather waiting threads to be redistributed into the buckets of the new table.
		// Threads with the same address remain in the same relative order as they were queued.
		TArray<FThread*, TInlineAllocator<128, FScratchAllocator>> Threads;
		for (FBucket* Bucket : ExistingBuckets)
		{
			while (FThread* Thread = Bucket->Dequeue())
			{
				Threads.Add(Thread);
			}
		}

		FTable& NewTable = Create(TargetBucketCount);

		// Reuse existing now-empty buckets when populating the new table.
		TArray<FBucket*, FScratchAllocator> AvailableBuckets = ExistingBuckets;
		const auto AllocateBucket = [&AvailableBuckets]() -> FBucket*
		{
			return !AvailableBuckets.IsEmpty() ? AvailableBuckets.Pop(EAllowShrinking::No) : FBucket::Create();
		};

		// Add waiting threads to the new table.
		for (FThread* Thread : Threads)
		{
			const uint32 Hash = HashAddress(Thread->WaitAddress.load(std::memory_order_relaxed));
			const uint32 Index = Hash % NewTable.BucketCount;
			FBucket& Bucket = NewTable.FindOrCreateBucket(Index, AllocateBucket);
			Bucket.Enqueue(Thread);
		}

		// Assign any available buckets to the table to avoid having to free them.
		for (uint32 Index = 0; !AvailableBuckets.IsEmpty() && Index < NewTable.BucketCount; ++Index)
		{
			NewTable.FindOrCreateBucket(Index, AllocateBucket);
		}
		checkSlow(AvailableBuckets.IsEmpty());

		// Make the new table visible to other threads.
		FTable* CompareTable = GlobalTable.exchange(&NewTable, std::memory_order_release);
		checkSlow(CompareTable == &ExistingTable);

		// Unlock buckets that came from the existing table now that the new table is visible.
		Unlock(ExistingBuckets);
		return;
	}
}

bool FTable::TryLock(FTable& Table, TArray<FBucket*, FScratchAllocator>& OutBuckets)
{
	OutBuckets.Reset(Table.BucketCount);

	// Gather buckets from the table, creating them as needed because the lock is on the bucket.
	for (uint32 Index = 0; Index < Table.BucketCount; ++Index)
	{
		OutBuckets.Add(&Table.FindOrCreateBucket(Index, [] { return FBucket::Create(); }));
	}

	// Lock the buckets in order by address to ensure consistent ordering regardless of the table being locked.
	Algo::Sort(OutBuckets);
	for (FBucket* Bucket : OutBuckets)
	{
		Bucket->Lock();
	}

	// Table is locked if the global table pointer still points to it, otherwise it has grown.
	if (&Table == GlobalTable)
	{
		return true;
	}

	// Unlock and return that the table could not be locked.
	Unlock(OutBuckets);
	OutBuckets.Reset();
	return false;
}

void FTable::Unlock(TConstArrayView<FBucket*> LockedBuckets)
{
	for (FBucket* Bucket : LockedBuckets)
	{
		Bucket->Unlock();
	}
}

uint32 FTable::HashAddress(const void* Address)
{
	constexpr uint64 A = 0xdc2b17dc9d2fbc29;
	constexpr uint64 B = 0xcb1014192cb2c5fc;
	constexpr uint64 C = 0x5b12db9242bd7ce7;
	const UPTRINT Value = UPTRINT(Address);
	return uint32(((A * (Value >> 32)) + (B * (Value & 0xffffffff)) + C) >> 32);
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FThreadLocalData::FThreadLocalData()
{
	FTable::Reserve(ThreadCount.fetch_add(1, std::memory_order_relaxed) + 1);
}

FThreadLocalData::~FThreadLocalData()
{
	ThreadCount.fetch_sub(1, std::memory_order_relaxed);
	Thread.SafeRelease();
	bDestroyed = true;
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

UE_AUTORTFM_NOAUTORTFM
FWaitState Wait(const void* Address, bool (*CanWait)(void*), void* CanWaitContext, void (*BeforeWait)(void*), void* BeforeWaitContext)
{
	using namespace Private;

	// A failure of this assertion is either an error in a callback, because callbacks may not call into ParkingLot,
	// or a dependency on ParkingLot from within the memory allocator used by ParkingLot.
	checkf(ParkingLotExecutingFunction == EExecutingFunction::None,
		TEXT("Detected a re-entrant call into ParkingLot from within a %s callback."), LexToString(ParkingLotExecutingFunction));

	TRefCountPtr<FThread> Self = FThreadLocalData::Get();

	checkfSlow(!Self->WaitAddress.load(std::memory_order_relaxed), TEXT("WaitAddress must be null. This can happen if Wait is called by BeforeWait."));
	checkfSlow(Self->WakeToken == 0, TEXT("WakeToken must be 0. This is an error in ParkingLot."));

	FWaitState State;

	// Enqueue the thread if CanWait returns true while the bucket is locked.
	{
		TDynamicUniqueLock<FWordMutex> BucketLock;
		FBucket& Bucket = FTable::FindOrCreateBucket(Address, BucketLock);
		if (CanWait)
		{
			TGuardValue GuardActive(ParkingLotExecutingFunction, EExecutingFunction::CanWait);
			if (!CanWait(CanWaitContext))
			{
				return State;
			}
		}
		State.bDidWait = true;
		Self->WaitAddress.store(Address, std::memory_order_relaxed);
		Self->Event.Reset();
		Bucket.Enqueue(&*Self);
	}

	// BeforeWait must be invoked after the bucket is unlocked.
	if (BeforeWait)
	{
		TGuardValue GuardActive(ParkingLotExecutingFunction, EExecutingFunction::BeforeWait);
		BeforeWait(BeforeWaitContext);
	}

	// Wait until the thread has been dequeued.
	Self->Event.Wait();

	// WaitAddress is reset when the thread is dequeued.
	checkSlow(!Self->WaitAddress.load(std::memory_order_relaxed));
	State.bDidWake = true;
	State.WakeToken = Self->WakeToken;
	Self->WakeToken = 0;
	return State;
}

static FWaitState TimedWait(
	const void* Address,
	bool (*CanWait)(void*),
	void* CanWaitContext,
	void (*BeforeWait)(void*),
	void* BeforeWaitContext,
	void (*WaitOnEvent)(void*, FPlatformManualResetEvent&),
	void* WaitOnEventContext)
{
	using namespace Private;

	// A failure of this assertion is either an error in a callback, because callbacks may not call into ParkingLot,
	// or a dependency on ParkingLot from within the memory allocator used by ParkingLot.
	checkf(ParkingLotExecutingFunction == EExecutingFunction::None,
		TEXT("Detected a re-entrant call into ParkingLot from within a %s callback."), LexToString(ParkingLotExecutingFunction));

	TRefCountPtr<FThread> Self = FThreadLocalData::Get();

	checkfSlow(!Self->WaitAddress.load(std::memory_order_relaxed), TEXT("WaitAddress must be null. This can happen if Wait is called by BeforeWait."));
	checkfSlow(Self->WakeToken == 0, TEXT("WakeToken must be 0. This is an error in ParkingLot."));

	FWaitState State;

	// Enqueue the thread if CanWait returns true while the bucket is locked.
	{
		TDynamicUniqueLock<FWordMutex> BucketLock;
		FBucket& Bucket = FTable::FindOrCreateBucket(Address, BucketLock);
		if (CanWait)
		{
			TGuardValue GuardActive(ParkingLotExecutingFunction, EExecutingFunction::CanWait);
			if (!CanWait(CanWaitContext))
			{
				return State;
			}
		}
		State.bDidWait = true;
		Self->WaitAddress.store(Address, std::memory_order_relaxed);
		Self->Event.Reset();
		Bucket.Enqueue(&*Self);
	}

	// BeforeWait must be invoked after the bucket is unlocked.
	if (BeforeWait)
	{
		TGuardValue GuardActive(ParkingLotExecutingFunction, EExecutingFunction::BeforeWait);
		BeforeWait(BeforeWaitContext);
	}

	// Wait until the timeout or until the thread has been dequeued.
	WaitOnEvent(WaitOnEventContext, Self->Event);

	// WaitAddress is reset when the thread is dequeued.
	if (!Self->WaitAddress.load(std::memory_order_relaxed))
	{
		State.bDidWake = true;
		State.WakeToken = Self->WakeToken;
		Self->WakeToken = 0;
		return State;
	}

	// The timeout was reached and the thread needs to dequeue itself.
	// This can race with a call to wake a thread, which means Self is unsafe to access outside of the lock.
	bool bDequeued = false;
	if (TDynamicUniqueLock<FWordMutex> BucketLock; FBucket* Bucket = FTable::FindBucket(Address, BucketLock))
	{
		Bucket->DequeueIf([Self = &*Self, &bDequeued](FThread* Thread)
		{
			if (Thread == Self)
			{
				bDequeued = true;
				Thread->WaitAddress.store(nullptr, std::memory_order_relaxed);
				return EQueueAction::RemoveAndStop;
			}
			return EQueueAction::Continue;
		});
	}

	// The thread did not dequeue itself, which means that we need to wait until the other thread
	// has finished waking this thread by setting its wait address to null.
	if (!bDequeued)
	{
		Self->Event.Wait();
		State.bDidWake = true;
		State.WakeToken = Self->WakeToken;
		Self->WakeToken = 0;
	}

	return State;
}

static void WaitForTime(void* WaitTime, FPlatformManualResetEvent& Event)
{
	Event.WaitFor(*(FMonotonicTimeSpan*)WaitTime);
}

UE_AUTORTFM_NOAUTORTFM
FWaitState WaitFor(const void* Address, bool (*CanWait)(void*), void* CanWaitContext, void (*BeforeWait)(void*), void* BeforeWaitContext, FMonotonicTimeSpan WaitTime)
{
	check(!WaitTime.IsNaN());
	return TimedWait(Address, CanWait, CanWaitContext, BeforeWait, BeforeWaitContext, WaitForTime, &WaitTime);
}

static void WaitUntilTime(void* WaitTime, FPlatformManualResetEvent& Event)
{
	Event.WaitUntil(*(FMonotonicTimePoint*)WaitTime);
}

UE_AUTORTFM_NOAUTORTFM
FWaitState WaitUntil(const void* Address, bool (*CanWait)(void*), void* CanWaitContext, void (*BeforeWait)(void*), void* BeforeWaitContext, FMonotonicTimePoint WaitTime)
{
	check(!WaitTime.IsNaN());
	return TimedWait(Address, CanWait, CanWaitContext, BeforeWait, BeforeWaitContext, WaitUntilTime, &WaitTime);
}

UE_AUTORTFM_NOAUTORTFM
void WakeOne(const void* Address, uint64 (*OnWakeState)(void*, FWakeState), void* OnWakeStateContext)
{
	using namespace Private;

	// A failure of this assertion is either an error in a callback, because callbacks may not call into ParkingLot,
	// or a dependency on ParkingLot from within the memory allocator used by ParkingLot.
	checkf(ParkingLotExecutingFunction <= EExecutingFunction::BeforeWait,
		TEXT("Detected a re-entrant call into ParkingLot from within a %s callback."), LexToString(ParkingLotExecutingFunction));

	TRefCountPtr<FThread> WakeThread;
	uint64 WakeToken = 0;

	{
		TDynamicUniqueLock<FWordMutex> BucketLock;
		FBucket& Bucket = FTable::FindOrCreateBucket(Address, BucketLock);
		Bucket.DequeueIf([Address, &WakeThread](FThread* Thread)
		{
			if (Thread->WaitAddress.load(std::memory_order_relaxed) == Address)
			{
				WakeThread = Thread;
				return EQueueAction::RemoveAndStop;
			}
			return EQueueAction::Continue;
		});
		TGuardValue GuardActive(ParkingLotExecutingFunction, EExecutingFunction::OnWakeState);
		FWakeState WakeState;
		WakeState.bDidWake = !!WakeThread;
		WakeState.bHasWaitingThreads = !Bucket.IsEmpty();
		WakeToken = OnWakeState ? OnWakeState(OnWakeStateContext, WakeState) : 0;
	}

	if (WakeThread)
	{
		WakeThread->WakeToken = WakeToken;
		WakeThread->WaitAddress.store(nullptr, std::memory_order_relaxed);
		WakeThread->Event.Notify();
	}
}

static uint64 CopyWakeState(void* OutState, FWakeState State)
{
	*(FWakeState*)OutState = State;
	return 0;
}

} // UE::ParkingLot::Private

namespace UE::ParkingLot
{

FWakeState WakeOne(const void* Address)
{
	FWakeState OutState;
	Private::WakeOne(Address, Private::CopyWakeState, &OutState);
	return OutState;
}

UE_AUTORTFM_NOAUTORTFM
uint32 WakeMultiple(const void* const Address, const uint32 WakeCount)
{
	using namespace Private;

	// A failure of this assertion is either an error in a callback, because callbacks may not call into ParkingLot,
	// or a dependency on ParkingLot from within the memory allocator used by ParkingLot.
	checkf(ParkingLotExecutingFunction <= EExecutingFunction::BeforeWait,
		TEXT("Detected a re-entrant call into ParkingLot from within a %s callback."), LexToString(ParkingLotExecutingFunction));

	// Reference the allocator to allow it to decommit scratch memory.
	TRefCountPtr ScratchAllocatorRef(&ParkingLotScratchAllocator);
	TArray<TRefCountPtr<FThread>, TInlineAllocator<128, FScratchAllocator>> WakeThreads;

	if (TDynamicUniqueLock<FWordMutex> BucketLock; FBucket* Bucket = FTable::FindBucket(Address, BucketLock))
	{
		Bucket->DequeueIf([Address, &WakeThreads, WakeCount](FThread* Thread)
		{
			if (Thread->WaitAddress.load(std::memory_order_relaxed) == Address)
			{
				const uint32 Count = uint32(WakeThreads.Add(Thread) + 1);
				return Count == WakeCount ? EQueueAction::RemoveAndStop : EQueueAction::RemoveAndContinue;
			}
			return EQueueAction::Continue;
		});
	}

	for (FThread* WakeThread : WakeThreads)
	{
		WakeThread->WaitAddress.store(nullptr, std::memory_order_relaxed);
		WakeThread->Event.Notify();
	}

	return uint32(WakeThreads.Num());
}

uint32 WakeAll(const void* Address)
{
	return WakeMultiple(Address, MAX_uint32);
}

} // UE::ParkingLot
