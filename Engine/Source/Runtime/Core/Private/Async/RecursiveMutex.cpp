// Copyright Epic Games, Inc. All Rights Reserved.

#include "Async/RecursiveMutex.h"

#include "Async/Fundamental/Oversubscription.h"
#include "Async/ParkingLot.h"
#include "HAL/PlatformProcess.h"
#include "HAL/PlatformTLS.h"

namespace UE
{

bool FRecursiveMutex::IsLockedByCurrentThread() const
{
	UE_AUTORTFM_REPORT_HAZARD_IF_CLOSED(RecursiveMutex);

	const uint32 CurrentThreadId = FPlatformTLS::GetCurrentThreadId();
	return ThreadId.load(std::memory_order_relaxed) == CurrentThreadId;
}

bool FRecursiveMutex::TryLock()
{
	UE_AUTORTFM_REPORT_HAZARD_IF_CLOSED(RecursiveMutex);

	const uint32 CurrentThreadId = FPlatformTLS::GetCurrentThreadId();
	uint32 CurrentState = State.load(std::memory_order_relaxed);

	// Try to acquire the lock if it was unlocked, even if there are waiting threads.
	// Acquiring the lock despite the waiting threads means that this lock is not FIFO and thus not fair.
	if (LIKELY(!(CurrentState & LockCountMask)))
	{
		if (LIKELY(State.compare_exchange_strong(CurrentState, CurrentState | (1 << LockCountShift), std::memory_order_acquire, std::memory_order_relaxed)))
		{
			checkSlow(ThreadId.load(std::memory_order_relaxed) == 0);
			ThreadId.store(CurrentThreadId, std::memory_order_relaxed);
			return true;
		}
	}

	// Lock recursively if this is the thread that holds the lock.
	if (ThreadId.load(std::memory_order_relaxed) == CurrentThreadId)
	{
		State.fetch_add(1 << LockCountShift, std::memory_order_relaxed);
		return true;
	}

	return false;
}

void FRecursiveMutex::Lock()
{
	UE_AUTORTFM_REPORT_HAZARD_IF_CLOSED(RecursiveMutex);

	const uint32 CurrentThreadId = FPlatformTLS::GetCurrentThreadId();
	uint32 CurrentState = State.load(std::memory_order_relaxed);

	// Try to acquire the lock if it was unlocked, even if there are waiting threads.
	// Acquiring the lock despite the waiting threads means that this lock is not FIFO and thus not fair.
	if (LIKELY(!(CurrentState & LockCountMask)))
	{
		if (LIKELY(State.compare_exchange_weak(CurrentState, CurrentState | (1 << LockCountShift), std::memory_order_acquire, std::memory_order_relaxed)))
		{
			checkSlow(ThreadId.load(std::memory_order_relaxed) == 0);
			ThreadId.store(CurrentThreadId, std::memory_order_relaxed);
			return;
		}
	}

	// Lock recursively if this is the thread that holds the lock.
	if (ThreadId.load(std::memory_order_relaxed) == CurrentThreadId)
	{
		State.fetch_add(1 << LockCountShift, std::memory_order_relaxed);
		return;
	}

	LockSlow(CurrentState, CurrentThreadId);
}

FORCENOINLINE void FRecursiveMutex::LockSlow(uint32 CurrentState, const uint32 CurrentThreadId)
{
	constexpr int32 SpinLimit = 40;
	int32 SpinCount = 0;
	for (;;)
	{
		// Try to acquire the lock if it was unlocked, even if there are waiting threads.
		// Acquiring the lock despite the waiting threads means that this lock is not FIFO and thus not fair.
		if (LIKELY(!(CurrentState & LockCountMask)))
		{
			if (LIKELY(State.compare_exchange_weak(CurrentState, CurrentState | (1 << LockCountShift), std::memory_order_acquire, std::memory_order_relaxed)))
			{
				checkSlow(ThreadId.load(std::memory_order_relaxed) == 0);
				ThreadId.store(CurrentThreadId, std::memory_order_relaxed);
				return;
			}
			continue;
		}

		// Spin up to the spin limit while there are no waiting threads.
		if (LIKELY(!(CurrentState & MayHaveWaitingLockFlag) && SpinCount < SpinLimit))
		{
			FPlatformProcess::Yield();
			++SpinCount;
			CurrentState = State.load(std::memory_order_relaxed);
			continue;
		}

		// Store that there are waiting threads. Restart if the state has changed since it was loaded.
		if (LIKELY(!(CurrentState & MayHaveWaitingLockFlag)))
		{
			if (UNLIKELY(!State.compare_exchange_weak(CurrentState, CurrentState | MayHaveWaitingLockFlag, std::memory_order_relaxed)))
			{
				continue;
			}
			CurrentState |= MayHaveWaitingLockFlag;
		}

		// Do not enter oversubscription during a wait on a mutex since the wait is generally too short
		// for it to matter and it can worsen performance a lot for heavily contended locks.
		LowLevelTasks::Private::FOversubscriptionAllowedScope _(false);

		// Wait if the state has not changed. Either way, loop back and try to acquire the lock after trying to wait.
		ParkingLot::Wait(&State, [this, CurrentState] { return State.load(std::memory_order_relaxed) == CurrentState; }, nullptr);
		CurrentState = State.load(std::memory_order_relaxed);
	}
}

void FRecursiveMutex::Unlock()
{
	UE_AUTORTFM_REPORT_HAZARD_IF_CLOSED(RecursiveMutex);

	uint32 CurrentState = State.load(std::memory_order_relaxed);
	checkSlow(CurrentState & LockCountMask);
	checkSlow(ThreadId.load(std::memory_order_relaxed) == FPlatformTLS::GetCurrentThreadId());

	if (LIKELY((CurrentState & LockCountMask) == (1 << LockCountShift)))
	{
		// Remove the association with this thread before unlocking.
		ThreadId.store(0, std::memory_order_relaxed);

		// Unlock immediately to allow other threads to acquire the lock while this thread looks for a thread to wake.
		const uint32 LastState = State.fetch_sub(1 << LockCountShift, std::memory_order_release);

		// Wake one exclusive waiter if there are waiting threads.
		if (UNLIKELY(LastState & MayHaveWaitingLockFlag))
		{
			WakeWaitingThread();
		}
	}
	else
	{
		// This is recursively locked. Decrement the lock count.
		State.fetch_sub(1 << LockCountShift, std::memory_order_relaxed);
	}
}

FORCENOINLINE void FRecursiveMutex::WakeWaitingThread()
{
	ParkingLot::WakeOne(&State, [this](ParkingLot::FWakeState WakeState) -> uint64
	{
		if (!WakeState.bHasWaitingThreads)
		{
			State.fetch_and(~MayHaveWaitingLockFlag, std::memory_order_relaxed);
		}
		return 0;
	});
}

} // UE
