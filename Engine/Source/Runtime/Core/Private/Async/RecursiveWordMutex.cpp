// Copyright Epic Games, Inc. All Rights Reserved.

#include "Async/RecursiveWordMutex.h"
#include "HAL/PlatformTLS.h"

namespace UE
{

bool FRecursiveWordMutex::IsLockedByCurrentThread() const
{
	const uint32 CurrentThreadId = FPlatformTLS::GetCurrentThreadId();
	return ThreadId.load(std::memory_order_relaxed) == CurrentThreadId;
}

bool FRecursiveWordMutex::TryLock()
{
	const uint32 CurrentThreadId = FPlatformTLS::GetCurrentThreadId();
	if (ThreadId.load(std::memory_order_relaxed) == CurrentThreadId)
	{
		++RecursionCount;
		return true;
	}
	else if (Mutex.TryLock())
	{
		ThreadId.store(CurrentThreadId, std::memory_order_relaxed);
		return true;
	}
	return false;
}

void FRecursiveWordMutex::Lock()
{
	const uint32 CurrentThreadId = FPlatformTLS::GetCurrentThreadId();
	if (ThreadId.load(std::memory_order_relaxed) == CurrentThreadId)
	{
		++RecursionCount;
	}
	else
	{
		Mutex.Lock();
		ThreadId.store(CurrentThreadId, std::memory_order_relaxed);
	}
}

void FRecursiveWordMutex::Unlock()
{
	if (RecursionCount > 0)
	{
		--RecursionCount;
	}
	else
	{
		ThreadId.store(0, std::memory_order_relaxed);
		Mutex.Unlock();
	}
}

} // UE
