// Copyright Epic Games, Inc. All Rights Reserved.

#include "Async/SharedMutex.h"

#include "Async/Fundamental/Oversubscription.h"
#include "Async/IntrusiveMutex.h"
#include "Async/ParkingLot.h"
#include "HAL/PlatformProcess.h"

namespace UE
{

struct FSharedMutex::FParams
{
	constexpr static uint32 IsLockedFlag = FSharedMutex::IsLockedFlag;
	constexpr static uint32 IsLockedMask = FSharedMutex::IsLockedFlag | FSharedMutex::SharedLockCountMask;
	constexpr static uint32 MayHaveWaitingLockFlag = FSharedMutex::MayHaveWaitingLockFlag;

	inline static const void* GetWaitAddress(const std::atomic<uint32>& State)
	{
		return &State;
	}
};

inline const void* FSharedMutex::GetSharedLockAddress() const
{
	// Shared locks need a distinct address from exclusive locks to allow threads waiting for exclusive ownership
	// to be woken up without waking any threads waiting for shared ownership.
	return (const uint8*)&State + 1;
}

void FSharedMutex::LockSlow()
{
	TIntrusiveMutex<FParams>::LockLoop(State);
}

void FSharedMutex::LockSharedSlow()
{
	constexpr int32 SpinLimit = 40;
	int32 SpinCount = 0;
	for (uint32 CurrentState = State.load(std::memory_order_relaxed);;)
	{
		// Try to acquire the lock if it is unlocked and there are no waiting threads.
		if (LIKELY(!(CurrentState & (IsLockedFlag | MayHaveWaitingLockFlag))))
		{
			if (LIKELY(State.compare_exchange_weak(CurrentState, CurrentState + (1 << SharedLockCountShift), std::memory_order_acquire, std::memory_order_relaxed)))
			{
				return;
			}
			continue;
		}

		// Spin up to the spin limit while there are no waiting threads.
		if (LIKELY(!(CurrentState & MayHaveWaitingLockFlag) && SpinCount < SpinLimit))
		{
			FPlatformProcess::Yield();
			++SpinCount;
			CurrentState = State.load(std::memory_order_relaxed);
			continue;
		}

		// Store that there are waiting threads. Restart if the state has changed since it was loaded.
		if (LIKELY(!(CurrentState & MayHaveWaitingSharedLockFlag)))
		{
			if (UNLIKELY(!State.compare_exchange_weak(CurrentState, CurrentState | MayHaveWaitingSharedLockFlag, std::memory_order_relaxed)))
			{
				continue;
			}
			CurrentState |= MayHaveWaitingSharedLockFlag;
		}

		// Do not enter oversubscription during a wait on a mutex since the wait is generally too short
		// for it to matter and it can worsen performance a lot for heavily contended locks.
		LowLevelTasks::Private::FOversubscriptionAllowedScope _(false);

		// Wait if the state has not changed. Either way, loop back and try to acquire the lock after trying to wait.
		ParkingLot::Wait(GetSharedLockAddress(), [this, CurrentState]
		{
			return State.load(std::memory_order_relaxed) == CurrentState;
		}, nullptr);
		CurrentState = State.load(std::memory_order_relaxed);
	}
}

FORCENOINLINE void FSharedMutex::WakeWaitingThread()
{
	TIntrusiveMutex<FParams>::WakeWaitingThread(State);
}

FORCENOINLINE void FSharedMutex::WakeWaitingThreads(uint32 LastState)
{
	if (LastState & MayHaveWaitingLockFlag)
	{
		// Wake one thread that is waiting to acquire an exclusive lock.
		if (TIntrusiveMutex<FParams>::TryWakeWaitingThread(State))
		{
			return;
		}

		// Reload the state if there were no shared waiters because new
		// ones may have registered themselves since LastState was read.
		if (!(LastState & MayHaveWaitingSharedLockFlag))
		{
			LastState = State.load(std::memory_order_relaxed);
		}
	}

	if (LastState & MayHaveWaitingSharedLockFlag)
	{
		// Wake every thread that is waiting to acquire a shared lock.
		// The awoken threads might race against other exclusive locks.
		if (State.fetch_and(~MayHaveWaitingSharedLockFlag, std::memory_order_relaxed) & MayHaveWaitingSharedLockFlag)
		{
			ParkingLot::WakeAll(GetSharedLockAddress());
		}
	}
}

} // UE
