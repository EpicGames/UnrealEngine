// Copyright Epic Games, Inc. All Rights Reserved.

#include "Async/SharedRecursiveMutex.h"

#include "Async/Fundamental/Oversubscription.h"
#include "Async/ParkingLot.h"
#include "HAL/PlatformProcess.h"
#include "HAL/PlatformTLS.h"

namespace UE::Core::Private
{

struct FSharedRecursiveMutexStack
{
	constexpr FSharedRecursiveMutexStack() = default;

	~FSharedRecursiveMutexStack()
	{
		checkf(!Top, TEXT("Thread %u destroyed while holding a shared lock on the FSharedRecursiveMutex at 0x%p."),
			FPlatformTLS::GetCurrentThreadId(), Top->OwnedMutex);
	}

	FSharedRecursiveMutexLink* Top = nullptr;
};

static thread_local FSharedRecursiveMutexStack ThreadLocalSharedLocks;

bool FSharedRecursiveMutexLink::Owns(const FSharedRecursiveMutex* Mutex)
{
	for (FSharedRecursiveMutexLink* It = ThreadLocalSharedLocks.Top; It; It = It->Next)
	{
		if (It->OwnedMutex == Mutex)
		{
			return true;
		}
	}
	return false;
}

void FSharedRecursiveMutexLink::Push(const FSharedRecursiveMutex* Mutex)
{
	checkSlow(!OwnedMutex && !Next);
	OwnedMutex = Mutex;
	Next = ThreadLocalSharedLocks.Top;
	ThreadLocalSharedLocks.Top = this;
}

void FSharedRecursiveMutexLink::Pop()
{
	checkSlow(OwnedMutex);
	for (FSharedRecursiveMutexLink** Link = &ThreadLocalSharedLocks.Top; *Link; Link = &(*Link)->Next)
	{
		if ((*Link) == this)
		{
			*Link = Next;
			OwnedMutex = nullptr;
			Next = nullptr;
			return;
		}
	}
}

} // UE::Core::Private

namespace UE
{

inline const void* FSharedRecursiveMutex::GetLockAddress() const
{
	return &State;
}

inline const void* FSharedRecursiveMutex::GetSharedLockAddress() const
{
	// Shared locks need a distinct address from exclusive locks to allow threads waiting for exclusive ownership
	// to be woken up without waking any threads waiting for shared ownership.
	return (const uint8*)&State + 1;
}

bool FSharedRecursiveMutex::IsLockedByCurrentThread() const
{
	const uint32 CurrentThreadId = FPlatformTLS::GetCurrentThreadId();
	return ThreadId.load(std::memory_order_relaxed) == CurrentThreadId;
}

bool FSharedRecursiveMutex::TryLock()
{
	const uint32 CurrentThreadId = FPlatformTLS::GetCurrentThreadId();
	uint32 CurrentState = State.load(std::memory_order_relaxed);

	// Try to acquire the lock if it was unlocked, even if there are waiting threads.
	// Acquiring the lock despite the waiting threads means that this lock is not FIFO and thus not fair.
	if (LIKELY(!(CurrentState & (LockCountMask | SharedLockCountMask))))
	{
		if (LIKELY(State.compare_exchange_strong(CurrentState, CurrentState | (1 << LockCountShift), std::memory_order_acquire, std::memory_order_relaxed)))
		{
			checkSlow(ThreadId.load(std::memory_order_relaxed) == 0);
			ThreadId.store(CurrentThreadId, std::memory_order_relaxed);
			return true;
		}
	}

	// Lock recursively if this is the thread that holds the lock.
	if (ThreadId.load(std::memory_order_relaxed) == CurrentThreadId)
	{
		checkSlow((CurrentState & LockCountMask) != LockCountMask);
		State.fetch_add(1 << LockCountShift, std::memory_order_relaxed);
		return true;
	}

	return false;
}

void FSharedRecursiveMutex::Lock()
{
	const uint32 CurrentThreadId = FPlatformTLS::GetCurrentThreadId();
	uint32 CurrentState = State.load(std::memory_order_relaxed);

	// Try to acquire the lock if it was unlocked, even if there are waiting threads.
	// Acquiring the lock despite the waiting threads means that this lock is not FIFO and thus not fair.
	if (LIKELY(!(CurrentState & (LockCountMask | SharedLockCountMask))))
	{
		if (LIKELY(State.compare_exchange_weak(CurrentState, CurrentState | (1 << LockCountShift), std::memory_order_acquire, std::memory_order_relaxed)))
		{
			checkSlow(ThreadId.load(std::memory_order_relaxed) == 0);
			ThreadId.store(CurrentThreadId, std::memory_order_relaxed);
			return;
		}
	}

	// Lock recursively if this is the thread that holds the lock.
	if (ThreadId.load(std::memory_order_relaxed) == CurrentThreadId)
	{
		checkSlow((CurrentState & LockCountMask) != LockCountMask);
		State.fetch_add(1 << LockCountShift, std::memory_order_relaxed);
		return;
	}

	LockSlow(CurrentState, CurrentThreadId);
}

FORCENOINLINE void FSharedRecursiveMutex::LockSlow(uint32 CurrentState, const uint32 CurrentThreadId)
{
	constexpr int32 SpinLimit = 40;
	int32 SpinCount = 0;
	for (;;)
	{
		// Try to acquire the lock if it was unlocked, even if there are waiting threads.
		// Acquiring the lock despite the waiting threads means that this lock is not FIFO and thus not fair.
		if (LIKELY(!(CurrentState & (LockCountMask | SharedLockCountMask))))
		{
			if (LIKELY(State.compare_exchange_weak(CurrentState, CurrentState | (1 << LockCountShift), std::memory_order_acquire, std::memory_order_relaxed)))
			{
				checkSlow(ThreadId.load(std::memory_order_relaxed) == 0);
				ThreadId.store(CurrentThreadId, std::memory_order_relaxed);
				return;
			}
			continue;
		}

		// Spin up to the spin limit while there are no waiting threads.
		if (LIKELY(!(CurrentState & MayHaveWaitingLockFlag) && SpinCount < SpinLimit))
		{
			FPlatformProcess::Yield();
			++SpinCount;
			CurrentState = State.load(std::memory_order_relaxed);
			continue;
		}

		// Store that there are waiting threads. Restart if the state has changed since it was loaded.
		if (LIKELY(!(CurrentState & MayHaveWaitingLockFlag)))
		{
			if (UNLIKELY(!State.compare_exchange_weak(CurrentState, CurrentState | MayHaveWaitingLockFlag, std::memory_order_relaxed)))
			{
				continue;
			}
			CurrentState |= MayHaveWaitingLockFlag;
		}

		// Do not enter oversubscription during a wait on a mutex since the wait is generally too short
		// for it to matter and it can worsen performance a lot for heavily contended locks.
		LowLevelTasks::Private::FOversubscriptionAllowedScope _(false);

		// Wait if the state has not changed. Either way, loop back and try to acquire the lock after trying to wait.
		ParkingLot::Wait(GetLockAddress(), [this, CurrentState]
		{
			return State.load(std::memory_order_relaxed) == CurrentState;
		}, nullptr);
		CurrentState = State.load(std::memory_order_relaxed);
	}
}

void FSharedRecursiveMutex::Unlock()
{
	uint32 CurrentState = State.load(std::memory_order_relaxed);
	checkSlow(CurrentState & LockCountMask);
	checkSlow(ThreadId.load(std::memory_order_relaxed) == FPlatformTLS::GetCurrentThreadId());

	if (LIKELY((CurrentState & LockCountMask) == (1 << LockCountShift)))
	{
		// Remove the association with this thread before unlocking.
		ThreadId.store(0, std::memory_order_relaxed);

		// Unlock immediately to allow other threads to acquire the lock while this thread looks for a thread to wake.
		uint32 LastState = State.fetch_sub(1 << LockCountShift, std::memory_order_release);

		// Wake one exclusive waiter or every shared waiter if there are waiting threads.
		if (UNLIKELY(LastState & (MayHaveWaitingLockFlag | MayHaveWaitingSharedLockFlag)))
		{
			WakeWaitingThreads(LastState);
		}
	}
	else
	{
		// This is recursively locked. Decrement the lock count.
		State.fetch_sub(1 << LockCountShift, std::memory_order_relaxed);
	}
}

bool FSharedRecursiveMutex::TryLockShared(Core::Private::FSharedRecursiveMutexLink& Link)
{
	uint32 CurrentState = State.load(std::memory_order_relaxed);
	// Recursive shared locks are quick to acquire. Check for that case and add 1 to the count.
	if ((CurrentState & SharedLockCountMask) && Core::Private::FSharedRecursiveMutexLink::Owns(this))
	{
		[[maybe_unused]] uint32 LastState = State.fetch_add(1 << SharedLockCountShift, std::memory_order_relaxed);
		checkSlow((LastState & SharedLockCountMask) != SharedLockCountMask);
		Link.Push(this);
		return true;
	}
	// Try to acquire a shared lock if there is no active or waiting exclusive lock.
	else
	{
		while (!(CurrentState & (LockCountMask | MayHaveWaitingLockFlag)))
		{
			checkSlow((CurrentState & SharedLockCountMask) != SharedLockCountMask);
			if (State.compare_exchange_weak(CurrentState, CurrentState + (1 << SharedLockCountShift), std::memory_order_acquire, std::memory_order_relaxed))
			{
				Link.Push(this);
				return true;
			}
		}
	}
	return false;
}

void FSharedRecursiveMutex::LockShared(Core::Private::FSharedRecursiveMutexLink& Link)
{
	uint32 CurrentState = State.load(std::memory_order_relaxed);
	// Recursive shared locks are quick to acquire. Check for that case and add 1 to the count.
	if ((CurrentState & SharedLockCountMask) && Core::Private::FSharedRecursiveMutexLink::Owns(this))
	{
		[[maybe_unused]] uint32 LastState = State.fetch_add(1 << SharedLockCountShift, std::memory_order_relaxed);
		checkSlow((LastState & SharedLockCountMask) != SharedLockCountMask);
		Link.Push(this);
		return;
	}
	// Try to acquire a shared lock if there is no active or waiting exclusive lock.
	else if (!(CurrentState & (LockCountMask | MayHaveWaitingLockFlag)))
	{
		checkSlow((CurrentState & SharedLockCountMask) != SharedLockCountMask);
		if (State.compare_exchange_weak(CurrentState, CurrentState + (1 << SharedLockCountShift), std::memory_order_acquire, std::memory_order_relaxed))
		{
			Link.Push(this);
			return;
		}
	}
	LockSharedSlow(Link);
}

FORCENOINLINE void FSharedRecursiveMutex::LockSharedSlow(Core::Private::FSharedRecursiveMutexLink& Link)
{
	constexpr int32 SpinLimit = 40;
	int32 SpinCount = 0;
	for (uint32 CurrentState = State.load(std::memory_order_relaxed);;)
	{
		// Try to acquire the lock if it is unlocked and there are no waiting threads.
		if (LIKELY(!(CurrentState & (LockCountMask | MayHaveWaitingLockFlag))))
		{
			checkSlow((CurrentState & SharedLockCountMask) != SharedLockCountMask);
			if (LIKELY(State.compare_exchange_weak(CurrentState, CurrentState + (1 << SharedLockCountShift), std::memory_order_acquire, std::memory_order_relaxed)))
			{
				Link.Push(this);
				return;
			}
			continue;
		}

		// Spin up to the spin limit while there are no waiting threads.
		if (LIKELY(!(CurrentState & MayHaveWaitingLockFlag) && SpinCount < SpinLimit))
		{
			FPlatformProcess::Yield();
			++SpinCount;
			CurrentState = State.load(std::memory_order_relaxed);
			continue;
		}

		// Store that there are waiting threads. Restart if the state has changed since it was loaded.
		if (LIKELY(!(CurrentState & MayHaveWaitingSharedLockFlag)))
		{
			if (UNLIKELY(!State.compare_exchange_weak(CurrentState, CurrentState | MayHaveWaitingSharedLockFlag, std::memory_order_relaxed)))
			{
				continue;
			}
			CurrentState |= MayHaveWaitingSharedLockFlag;
		}

		// Do not enter oversubscription during a wait on a mutex since the wait is generally too short
		// for it to matter and it can worsen performance a lot for heavily contended locks.
		LowLevelTasks::Private::FOversubscriptionAllowedScope _(false);

		// Wait if the state has not changed. Either way, loop back and try to acquire the lock after trying to wait.
		ParkingLot::Wait(GetSharedLockAddress(), [this, CurrentState]
		{
			return State.load(std::memory_order_relaxed) == CurrentState;
		}, nullptr);
		CurrentState = State.load(std::memory_order_relaxed);
	}
}

void FSharedRecursiveMutex::UnlockShared(Core::Private::FSharedRecursiveMutexLink& Link)
{
	Link.Pop();
	// C++ SharedMutex requires that UnlockShared() synchronizes-with subsequent Lock() operations.
	// This requirement necessitates the use of release memory order or stronger.
	const uint32 LastState = State.fetch_sub(1 << SharedLockCountShift, std::memory_order_release);
	checkSlow(LastState & SharedLockCountMask);
	constexpr uint32 WakeState = MayHaveWaitingLockFlag | (1 << SharedLockCountShift);
	if (UNLIKELY((LastState & ~MayHaveWaitingSharedLockFlag) == WakeState))
	{
		// The last shared lock was released and there is a waiting exclusive lock.
		// Wake one thread that is waiting to acquire an exclusive lock.
		ParkingLot::WakeOne(GetLockAddress(), [this](ParkingLot::FWakeState WakeState) -> uint64
		{
			if (!WakeState.bDidWake)
			{
				// Keep the flag until no thread wakes, otherwise shared locks may win before
				// an exclusive lock has a chance.
				State.fetch_and(~MayHaveWaitingLockFlag, std::memory_order_relaxed);
			}
			return 0;
		});
	}
}

FORCENOINLINE void FSharedRecursiveMutex::WakeWaitingThreads(uint32 LastState)
{
	if (LastState & MayHaveWaitingLockFlag)
	{
		// Wake one thread that is waiting to acquire an exclusive lock.
		bool bDidWake = false;
		ParkingLot::WakeOne(GetLockAddress(), [this, &bDidWake](ParkingLot::FWakeState WakeState) -> uint64
		{
			if (!WakeState.bDidWake)
			{
				// Keep the flag until no thread wakes, otherwise shared locks may win before
				// an exclusive lock has a chance.
				State.fetch_and(~MayHaveWaitingLockFlag, std::memory_order_relaxed);
			}
			bDidWake = WakeState.bDidWake;
			return 0;
		});
		if (bDidWake)
		{
			return;
		}

		// Reload the state if there were no shared waiters because new
		// ones may have registered themselves since LastState was read.
		if (!(LastState & MayHaveWaitingSharedLockFlag))
		{
			LastState = State.load(std::memory_order_relaxed);
		}
	}

	if (LastState & MayHaveWaitingSharedLockFlag)
	{
		// Wake every thread that is waiting to acquire a shared lock.
		// The awoken threads might race against other exclusive locks.
		if (State.fetch_and(~MayHaveWaitingSharedLockFlag, std::memory_order_relaxed) & MayHaveWaitingSharedLockFlag)
		{
			ParkingLot::WakeAll(GetSharedLockAddress());
		}
	}
}

} // UE
