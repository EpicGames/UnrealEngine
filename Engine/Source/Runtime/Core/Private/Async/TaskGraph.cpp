// Copyright Epic Games, Inc. All Rights Reserved.


#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "Math/NumericLimits.h"
#include "Math/UnrealMathUtility.h"
#include "HAL/UnrealMemory.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Logging/LogMacros.h"
#include "Misc/ScopedEvent.h"
#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "Misc/SingleThreadRunnable.h"
#include "HAL/ThreadSafeCounter.h"
#include "Misc/NoopCounter.h"
#include "Misc/ScopeLock.h"
#include "Async/ManualResetEvent.h"
#include "Containers/LockFreeList.h"
#include "Templates/Function.h"
#include "Stats/Stats.h"
#include "Misc/CoreStats.h"
#include "Math/RandomStream.h"
#include "HAL/IConsoleManager.h"
#include "Misc/App.h"
#include "Misc/Fork.h"
#include "Misc/Timeout.h"
#include "Async/TaskGraphInterfaces.h"
#include "HAL/LowLevelMemTracker.h"
#include "HAL/ThreadHeartBeat.h"
#include "ProfilingDebugging/ExternalProfiler.h"
#include "ProfilingDebugging/MiscTrace.h"
#include "ProfilingDebugging/ScopedTimers.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreDelegates.h"

#include "Async/Fundamental/Scheduler.h"
#include "Tasks/Pipe.h"

DEFINE_LOG_CATEGORY_STATIC(LogTaskGraph, Log, All);

DEFINE_STAT(STAT_FReturnGraphTask);
DEFINE_STAT(STAT_FTriggerEventGraphTask);
DEFINE_STAT(STAT_ParallelFor);
DEFINE_STAT(STAT_ParallelForTask);

static int32 GNumWorkerThreadsToIgnore = 0;

#if PLATFORM_USE_FULL_TASK_GRAPH && !IS_PROGRAM && WITH_ENGINE && !UE_SERVER
	#define CREATE_HIPRI_TASK_THREADS (1)
	#define CREATE_BACKGROUND_TASK_THREADS (1)
#else
	#define CREATE_HIPRI_TASK_THREADS (0)
	#define CREATE_BACKGROUND_TASK_THREADS (0)
#endif

#if !defined(YIELD_BETWEEN_TASKS)
	#define YIELD_BETWEEN_TASKS 0
#endif

namespace ENamedThreads
{
	CORE_API TAtomic<Type> FRenderThreadStatics::RenderThread(ENamedThreads::GameThread); // defaults to game and is set and reset by the render thread itself
	CORE_API TAtomic<Type> FRenderThreadStatics::RenderThread_Local(ENamedThreads::GameThread_Local); // defaults to game local and is set and reset by the render thread itself
	CORE_API int32 bHasBackgroundThreads = CREATE_BACKGROUND_TASK_THREADS;
	CORE_API int32 bHasHighPriorityThreads = CREATE_HIPRI_TASK_THREADS;
}

// RenderingThread.cpp sets these values if needed
CORE_API bool GRenderThreadPollingOn = false;	// Access/Modify on GT only. This value is set on the GT before actual state is changed on the RT.
CORE_API int32 GRenderThreadPollPeriodMs = -1;	// Access/Modify on RT only.

static int32 GIgnoreThreadToDoGatherOn = 0;
static FAutoConsoleVariableRef CVarIgnoreThreadToDoGatherOn(
	TEXT("TaskGraph.IgnoreThreadToDoGatherOn"),
	GIgnoreThreadToDoGatherOn,
	TEXT("DEPRECATED! If 1, then we ignore the hint provided with SetGatherThreadForDontCompleteUntil and just run it on AnyHiPriThreadHiPriTask.")
);

static int32 GTestDontCompleteUntilForAlreadyComplete = 1;
static FAutoConsoleVariableRef CVarTestDontCompleteUntilForAlreadyComplete(
	TEXT("TaskGraph.TestDontCompleteUntilForAlreadyComplete"),
	GTestDontCompleteUntilForAlreadyComplete,
	TEXT("If 1, then we before spawning a gather task, we just check if all of the subtasks are complete, and in that case we can skip the gather.")
);

CORE_API bool GAllowTaskGraphForkMultithreading = true;
static FAutoConsoleVariableRef CVarEnableForkedMultithreading(
	TEXT("TaskGraph.EnableForkedMultithreading"),
	GAllowTaskGraphForkMultithreading,
	TEXT("When false will prevent the task graph from running multithreaded on forked processes.")
);

static int32 CVar_ForkedProcess_MaxWorkerThreads = 2;
static FAutoConsoleVariableRef CVarForkedProcessMaxWorkerThreads(
	TEXT("TaskGraph.ForkedProcessMaxWorkerThreads"),
	CVar_ForkedProcess_MaxWorkerThreads,
	TEXT("Configures the number of worker threads a forked process should spawn if it allows multithreading.")
);

// NOTE: the ECVF_ReadOnly cvars must be set in ini files that are read before InitThreadConfig

CORE_API bool GTaskGraphUseDynamicPrioritization = 1;
static FAutoConsoleVariableRef CVarTaskDynamicPrioritization(
	TEXT("TaskGraph.UseDynamicPrioritization"),
	GTaskGraphUseDynamicPrioritization,
	TEXT("Adjust thread priority per-task so that higher priority tasks running on background threads can't be preempted as easily. Helps a lot under high load."),
	ECVF_ReadOnly
);

CORE_API float GTaskGraphOversubscriptionRatio = 2.0f;
static FAutoConsoleVariableRef CVarTaskOversubscriptionRatio(
	TEXT("TaskGraph.OversubscriptionRatio"),
	GTaskGraphOversubscriptionRatio,
	TEXT("Ratio used to compute the maximum numbers of workers allowed during oversubscription.\n")
	TEXT("You might need to increase that value depending on how many recursive waits the scheduled tasks may contain.\n")
	TEXT("The optimal scenario to strive for is using prerequisites to setup dependencies instead of waiting.\n")
	TEXT("Once none of the scheduled tasks contains waiting logic anymore, this can be set to 1.0f, which effectively deactivate the feature."),
	ECVF_ReadOnly
);

CORE_API bool GTaskGraphUseDynamicThreadCreation = (PLATFORM_DESKTOP == 1);
static FAutoConsoleVariableRef CVarTaskDynamicThreadCreation(
	TEXT("TaskGraph.UseDynamicThreadCreation"),
	GTaskGraphUseDynamicThreadCreation,
	TEXT("Allow threads to be created only when needed instead of at engine initialization."),
	ECVF_ReadOnly
);

CORE_API int32 GNumForegroundWorkers = 2;
static FAutoConsoleVariableRef CVarNumForegroundWorkers(
	TEXT("TaskGraph.NumForegroundWorkers"),
	GNumForegroundWorkers,
	TEXT("Configures the number of foreground worker threads. Requires the scheduler to be restarted to have an affect")
);

CORE_API int32 GTaskThreadStackSize = 0;
static FAutoConsoleVariableRef CVarTaskThreadStackSize(
	TEXT("TaskGraph.ThreadStackSize"),
	GTaskThreadStackSize,
	TEXT("Thread Stack Size, 0 means platform default")
);

#if UE_ASYNC_WORK_STEALING_QUEUE_USES_RETRY_LIMIT
CORE_API int32 GWorkStealingQueueRetryLimit = -1;
static FAutoConsoleVariableRef WorkStealingQueueRetryLimitCVar(
	TEXT("TaskGraph.WorkStealingQueue.RetryLimit"),
	GWorkStealingQueueRetryLimit,
	TEXT("The maximum number of times (-1 = practically infinite) that a work-stealing attempt can fail due to a taken slot before it is forced to yield"),
	ECVF_Default);
#endif

#if CREATE_HIPRI_TASK_THREADS || CREATE_BACKGROUND_TASK_THREADS
	static void ThreadSwitchForABTest(const TArray<FString>& Args)
	{
		if (Args.Num() == 2)
		{
#if CREATE_HIPRI_TASK_THREADS
			ENamedThreads::bHasHighPriorityThreads = !!FCString::Atoi(*Args[0]);
#endif
#if CREATE_BACKGROUND_TASK_THREADS
			ENamedThreads::bHasBackgroundThreads = !!FCString::Atoi(*Args[1]);
#endif
		}
		else
		{
			UE_LOGF(LogConsoleResponse, Display, "This command requires two arguments, both 0 or 1 to control the use of high priority and background priority threads, respectively.");
		}
		UE_LOGF(LogConsoleResponse, Display, "High priority task threads: %d    Background priority threads: %d", ENamedThreads::bHasHighPriorityThreads, ENamedThreads::bHasBackgroundThreads);
	}

	static FAutoConsoleCommand ThreadSwitchForABTestCommand(
		TEXT("TaskGraph.ABTestThreads"),
		TEXT("Takes two 0/1 arguments. Equivalent to setting TaskGraph.UseHiPriThreads and TaskGraph.UseBackgroundThreads, respectively. Packages as one command for use with the abtest command."),
		FConsoleCommandWithArgsDelegate::CreateStatic(&ThreadSwitchForABTest)
		);

#endif 


#if CREATE_BACKGROUND_TASK_THREADS
static FAutoConsoleVariableRef CVarUseBackgroundThreads(
	TEXT("TaskGraph.UseBackgroundThreads"),
	ENamedThreads::bHasBackgroundThreads,
	TEXT("If > 0, then use background threads, otherwise run background tasks on normal priority task threads. Used for performance tuning."),
	ECVF_Cheat
	);
#endif

#if CREATE_HIPRI_TASK_THREADS
static FAutoConsoleVariableRef CVarUseHiPriThreads(
	TEXT("TaskGraph.UseHiPriThreads"),
	ENamedThreads::bHasHighPriorityThreads,
	TEXT("If > 0, then use hi priority task threads, otherwise run background tasks on normal priority task threads. Used for performance tuning."),
	ECVF_Cheat
	);
#endif

#define PROFILE_TASKGRAPH (0)
#if PROFILE_TASKGRAPH
	struct FProfileRec
	{
		const TCHAR* Name;
		FThreadSafeCounter NumSamplesStarted;
		FThreadSafeCounter NumSamplesFinished;
		uint32 Samples[1000];

		FProfileRec()
		{
			Name = nullptr;
		}
	};
	static FThreadSafeCounter NumProfileSamples;
	static void DumpProfile();
	struct FProfileRecScope
	{
		FProfileRec* Target;
		int32 SampleIndex;
		uint32 StartCycles;
		FProfileRecScope(FProfileRec* InTarget, const TCHAR* InName)
			: Target(InTarget)
			, SampleIndex(InTarget->NumSamplesStarted.Increment() - 1)
			, StartCycles(FPlatformTime::Cycles())
		{
			if (SampleIndex == 0 && !Target->Name)
			{
				Target->Name = InName;
			}
		}
		~FProfileRecScope()
		{
			if (SampleIndex < 1000)
			{
				Target->Samples[SampleIndex] = FPlatformTime::Cycles() - StartCycles;
				if (Target->NumSamplesFinished.Increment() == 1000)
				{
					Target->NumSamplesFinished.Reset();
					FPlatformMisc::MemoryBarrier();
					uint64 Total = 0;
					for (int32 Index = 0; Index < 1000; Index++)
					{
						Total += Target->Samples[Index];
					}
					float MsPer = FPlatformTime::GetSecondsPerCycle() * double(Total);
					UE_LOGF(LogTemp, Display, "%6.4f ms / scope %ls",MsPer, Target->Name);

					Target->NumSamplesStarted.Reset();
				}
			}
		}
	};
	static FProfileRec ProfileRecs[10];
	static void DumpProfile()
	{

	}

	#define TASKGRAPH_SCOPE_CYCLE_COUNTER(Index, Name) \
		FProfileRecScope ProfileRecScope##Index(&ProfileRecs[Index], TEXT(#Name));


#else
	#define TASKGRAPH_SCOPE_CYCLE_COUNTER(Index, Name)
#endif

TArray<TaskTrace::FId> GetTraceIds(const FGraphEventArray& Tasks)
{
#if UE_TASK_TRACE_ENABLED
	TArray<TaskTrace::FId> TasksIds;
	TasksIds.Reserve(Tasks.Num());

	for (const FGraphEventRef& Task : Tasks)
	{
		if (Task.IsValid())
		{
			TasksIds.Add(Task->GetTraceId());
		}
	}

	return TasksIds;
#else
	return {};
#endif
}

/** 
 *	Pointer to the task graph implementation singleton.
 *	Because of the multithreaded nature of this system an ordinary singleton cannot be used.
 *	FTaskGraphImplementation::Startup() creates the singleton and the constructor actually sets this value.
**/
class FTaskGraphImplementation;
struct FWorkerThread;

static FTaskGraphInterface* TaskGraphImplementationSingleton = NULL;

#if !UE_BUILD_SHIPPING && !UE_BUILD_TEST

static struct FChaosMode 
{
	enum 
	{
		NumSamples = 45771
	};

	struct FState
	{
		FThreadSafeCounter Current;
		float DelayTimes[NumSamples + 1]; 
		int32 Enabled;

		FState()
			: Enabled(0)
		{
			FRandomStream Stream((int32)FPlatformTime::Cycles());
			for (int32 Index = 0; Index < NumSamples; Index++)
			{
				DelayTimes[Index] = Stream.GetFraction();
			}
			// ave = .5
			for (int32 Cube = 0; Cube < 2; Cube++)
			{
				for (int32 Index = 0; Index < NumSamples; Index++)
				{
					DelayTimes[Index] *= Stream.GetFraction();
				}
			}
			// ave = 1/8
			for (int32 Index = 0; Index < NumSamples; Index++)
			{
				DelayTimes[Index] *= 0.00001f;
			}
			// ave = 0.00000125s
			for (int32 Zeros = 0; Zeros < NumSamples / 20; Zeros++)
			{
				int32 Index = Stream.RandHelper(NumSamples);
				DelayTimes[Index] = 0.0f;
			}
			// 95% the samples are now zero
			for (int32 Zeros = 0; Zeros < NumSamples / 100; Zeros++)
			{
				int32 Index = Stream.RandHelper(NumSamples);
				DelayTimes[Index] = .00005f;
			}
			// .001% of the samples are 5ms
		}
	};
	std::atomic<FState*> State;

	~FChaosMode()
	{
		if (FState* LocalState = State.load(std::memory_order_relaxed))
		{
			delete LocalState;
		}
	}

	FORCEINLINE void Delay()
	{
		FState* LocalState = State.load(std::memory_order_acquire);
		if (LocalState && LocalState->Enabled)
		{
			uint32 MyIndex = (uint32)LocalState->Current.Increment();
			MyIndex %= NumSamples;
			float DelayS = LocalState->DelayTimes[MyIndex];
			if (DelayS > 0.0f)
			{
				FPlatformProcess::Sleep(DelayS);
			}
		}
	}
} GChaosMode;

static void EnableRandomizedThreads(const TArray<FString>& Args)
{
	FChaosMode::FState* LocalState = GChaosMode.State.load(std::memory_order_acquire);
	if (!LocalState)
	{
		LocalState = new FChaosMode::FState();
		GChaosMode.State.store(LocalState, std::memory_order_release);
	}
	LocalState->Enabled = !LocalState->Enabled;
	if (LocalState->Enabled)
	{
		UE_LOGF(LogConsoleResponse, Display, "Random sleeps are enabled.");
	}
	else
	{
		UE_LOGF(LogConsoleResponse, Display, "Random sleeps are disabled.");
	}
}

static FAutoConsoleCommand TestRandomizedThreadsCommand(
	TEXT("TaskGraph.Randomize"),
	TEXT("Useful for debugging, adds random sleeps throughout the task graph."),
	FConsoleCommandWithArgsDelegate::CreateStatic(&EnableRandomizedThreads)
	);

FORCEINLINE void TestRandomizedThreads()
{
	GChaosMode.Delay();
}

#else

FORCEINLINE void TestRandomizedThreads()
{
}

#endif

static TCHAR CharFromTaskPriority(ENamedThreads::Type InPriority)
{
	if (InPriority == ENamedThreads::HighThreadPriority)
	{
		return TEXT('h');
	}
	return TEXT('n');
}

static ENamedThreads::Type TaskPriorityFromChar(TCHAR InChar)
{
	if (InChar == TEXT('h'))
	{
		return ENamedThreads::HighTaskPriority;
	}
	return ENamedThreads::NormalTaskPriority;
}

static TCHAR CharFromThreadPriority(ENamedThreads::Type InPriority)
{
	if (InPriority == ENamedThreads::NormalThreadPriority)
	{
		return TEXT('n');
	}
	if (InPriority == ENamedThreads::HighThreadPriority)
	{
		return TEXT('h');
	}
	return TEXT('b');
}

static ENamedThreads::Type ThreadPriorityFromChar(TCHAR InChar)
{
	if (InChar == TEXT('n'))
	{
		return ENamedThreads::NormalThreadPriority;
	}
	if (InChar == TEXT('h'))
	{
		return ENamedThreads::HighThreadPriority;
	}
	return ENamedThreads::BackgroundThreadPriority;
}

FAutoConsoleTaskPriority::FAutoConsoleTaskPriority(const TCHAR* Name, const TCHAR* Help, ENamedThreads::Type DefaultThreadPriority, ENamedThreads::Type DefaultTaskPriority, ENamedThreads::Type DefaultTaskPriorityIfForcedToNormalThreadPriority)
	: RawSetting(ConfigStringFromPriorities(DefaultThreadPriority, DefaultTaskPriority, DefaultTaskPriorityIfForcedToNormalThreadPriority))
	, FullHelpText(CreateFullHelpText(Name, Help))
	, Variable(Name, RawSetting, *FullHelpText, FConsoleVariableDelegate::CreateRaw(this, &FAutoConsoleTaskPriority::OnSettingChanged), ECVF_Default)
	, ThreadPriority(DefaultThreadPriority)
	, TaskPriority(DefaultTaskPriority)
	, TaskPriorityIfForcedToNormalThreadPriority(DefaultTaskPriorityIfForcedToNormalThreadPriority)
{
	// if you are asking for a hi or background thread priority, you must provide a separate task priority to use if those threads are not available.
	check(TaskPriorityIfForcedToNormalThreadPriority != ENamedThreads::UnusedAnchor || ThreadPriority == ENamedThreads::NormalThreadPriority);
}

FString FAutoConsoleTaskPriority::CreateFullHelpText(const TCHAR* Name, const TCHAR* OriginalHelp)
{
	return FString::Printf(
		TEXT("%s\n"
		     "Arguments are three characters: [ThreadPriority][TaskPriority][TaskPriorityIfForcedToNormalThreadPriority] "
		     "where ThreadPriority is 'h' or 'n' or 'b' (high/normal/background) and TaskPriority is 'h' or 'n' (high/normal). "
		     "Example: %s bnh")
		, OriginalHelp, Name);
}

FString FAutoConsoleTaskPriority::ConfigStringFromPriorities(ENamedThreads::Type InThreadPriority, ENamedThreads::Type InTaskPriority, ENamedThreads::Type InTaskPriorityBackup)
{
	const TCHAR OutName[4] = {
		CharFromThreadPriority(InThreadPriority),
		CharFromTaskPriority(InTaskPriority),
		CharFromTaskPriority(InTaskPriorityBackup),
		TEXT('\0')
	};
	return FString(OutName);
}

void FAutoConsoleTaskPriority::OnSettingChanged(IConsoleVariable* InVariable)
{
	if (RawSetting.Len() > 0)
	{
		ThreadPriority = ThreadPriorityFromChar(RawSetting[0]);
	}
	if (RawSetting.Len() > 1)
	{
		TaskPriority = TaskPriorityFromChar(RawSetting[1]);
	}
	if (RawSetting.Len() > 2)
	{
		TaskPriorityIfForcedToNormalThreadPriority = TaskPriorityFromChar(RawSetting[2]);
	}
}

/** 
 *	FTaskThreadBase
 *	Base class for a thread that executes tasks
 *	This class implements the FRunnable API, but external threads don't use that because those threads are created elsewhere.
**/
class FTaskThreadBase : public FRunnable, FSingleThreadRunnable
{
public:
	// Calls meant to be called from a "main" or supervisor thread.

	/** Constructor, initializes everything to unusable values. Meant to be called from a "main" thread. **/
	FTaskThreadBase()
		: ThreadId(ENamedThreads::AnyThread)
		, PerThreadIDTLSSlot(FPlatformTLS::InvalidTlsSlot)
		, OwnerWorker(nullptr)
	{
		NewTasks.Reset(128);
	}

	/** 
	 *	Sets up some basic information for a thread. Meant to be called from a "main" thread. Also creates the stall event.
	 *	@param InThreadId; Thread index for this thread.
	 *	@param InPerThreadIDTLSSlot; TLS slot to store the pointer to me into (later)
	**/
	void Setup(ENamedThreads::Type InThreadId, uint32 InPerThreadIDTLSSlot, FWorkerThread* InOwnerWorker)
	{
		ThreadId = InThreadId;
		check(ThreadId >= 0);
		PerThreadIDTLSSlot = InPerThreadIDTLSSlot;
		OwnerWorker = InOwnerWorker;
	}

	// Calls meant to be called from "this thread".

	/** A one-time call to set the TLS entry for this thread. **/
	void InitializeForCurrentThread()
	{
		FPlatformTLS::SetTlsValue(PerThreadIDTLSSlot,OwnerWorker);
	}

	/** Return the index of this thread. **/
	ENamedThreads::Type GetThreadId() const
	{
		checkThreadGraph(OwnerWorker); // make sure we are started up
		return ThreadId;
	}

	/** Used for named threads to start processing tasks until the thread is idle and RequestQuit has been called. **/
	virtual void ProcessTasksUntilQuit(int32 QueueIndex) = 0;

	/** Used for named threads to start processing tasks until the thread is idle and RequestQuit has been called. **/
	virtual uint64 ProcessTasksUntilIdle(int32 QueueIndex)
	{
		check(0);
		return 0;
	}

	/** 
	 *	Queue a task, assuming that this thread is the same as the current thread.
	 *	For named threads, these go directly into the private queue.
	 *	@param QueueIndex, Queue to enqueue for
	 *	@param Task Task to queue.
	 **/
	virtual void EnqueueFromThisThread(int32 QueueIndex, FBaseGraphTask* Task)
	{
		check(0);
	}

	// Calls meant to be called from any thread.

	/** 
	 *	Will cause the thread to return to the caller when it becomes idle. Used to return from ProcessTasksUntilQuit for named threads or to shut down unnamed threads. 
	 *	CAUTION: This will not work under arbitrary circumstances. For example you should not attempt to stop unnamed threads unless they are known to be idle.
	 *	Return requests for named threads should be submitted from that named thread as FReturnGraphTask does.
	 *	@param QueueIndex, Queue to request quit from
	**/
	virtual void RequestQuit(int32 QueueIndex) = 0;

	/** 
	 *	Queue a task, assuming that this thread is not the same as the current thread.
	 *	@param QueueIndex, Queue to enqueue into
	 *	@param Task; Task to queue.
	 **/
	virtual bool EnqueueFromOtherThread(int32 QueueIndex, FBaseGraphTask* Task)
	{
		check(0);
		return false;
	}

	virtual void WakeUp(int32 QueueIndex = 0) = 0;

	/** 
	 *Return true if this thread is processing tasks. This is only a "guess" if you ask for a thread other than yourself because that can change before the function returns.
	 *@param QueueIndex, Queue to request quit from
	 **/
	virtual bool IsProcessingTasks(int32 QueueIndex) = 0;

	// SingleThreaded API

	/** Tick single-threaded. */
	virtual void Tick() override
	{
		ProcessTasksUntilIdle(0);
	}


	// FRunnable API

	/**
	 * Allows per runnable object initialization. NOTE: This is called in the
	 * context of the thread object that aggregates this, not the thread that
	 * passes this runnable to a new thread.
	 *
	 * @return True if initialization was successful, false otherwise
	 */
	virtual bool Init() override
	{
		InitializeForCurrentThread();
		return true;
	}

	/**
	 * This is where all per object thread work is done. This is only called
	 * if the initialization was successful.
	 *
	 * @return The exit code of the runnable object
	 */
	virtual uint32 Run() override
	{
		check(OwnerWorker); // make sure we are started up
		ProcessTasksUntilQuit(0);
		FMemory::ClearAndDisableTLSCachesOnCurrentThread();
		return 0;
	}

	/**
	 * This is called if a thread is requested to terminate early
	 */
	virtual void Stop() override
	{
		RequestQuit(-1);
	}

	/**
	 * Called in the context of the aggregating thread to perform any cleanup.
	 */
	virtual void Exit() override
	{
	}

	/**
	 * Return single threaded interface when multithreading is disabled.
	 */
	virtual FSingleThreadRunnable* GetSingleThreadInterface() override
	{
		return this;
	}

protected:

	/** Id / Index of this thread. **/
	ENamedThreads::Type									ThreadId;
	/** TLS SLot that we store the FTaskThread* this pointer in. **/
	uint32												PerThreadIDTLSSlot;
	/** Used to signal stalling. Not safe for synchronization in most cases. **/
	FThreadSafeCounter									IsStalled;
	/** Array of tasks for this task thread. */
	TArray<FBaseGraphTask*> NewTasks;
	/** back pointer to the owning FWorkerThread **/
	FWorkerThread* OwnerWorker;
};

/** 
 *	FNamedTaskThread
 *	A class for managing a named thread. 
 */
class FNamedTaskThread : public FTaskThreadBase
{
public:

	virtual void ProcessTasksUntilQuit(int32 QueueIndex) override
	{
		check(Queue(QueueIndex).StallRestartEvent); // make sure we are started up

		Queue(QueueIndex).QuitForReturn = false;
		verify(++Queue(QueueIndex).RecursionGuard == 1);
		const bool bIsMultiThread = FTaskGraphInterface::IsMultithread();
		do
		{
			const bool bAllowStall = bIsMultiThread;
			ProcessTasksNamedThread(QueueIndex, bAllowStall);
		} while (!Queue(QueueIndex).QuitForReturn && !Queue(QueueIndex).QuitForShutdown && bIsMultiThread); // @Hack - quit now when running with only one thread.
		verify(!--Queue(QueueIndex).RecursionGuard);
	}

	virtual uint64 ProcessTasksUntilIdle(int32 QueueIndex) override
	{
		check(Queue(QueueIndex).StallRestartEvent); // make sure we are started up

		Queue(QueueIndex).QuitForReturn = false;
		verify(++Queue(QueueIndex).RecursionGuard == 1);
		uint64 ProcessedTasks = ProcessTasksNamedThread(QueueIndex, false);
		verify(!--Queue(QueueIndex).RecursionGuard);
		return ProcessedTasks;
	}


	uint64 ProcessTasksNamedThread(int32 QueueIndex, bool bAllowStall)
	{
		uint64 ProcessedTasks = 0;
#if UE_EXTERNAL_PROFILING_ENABLED
		static thread_local bool bOnce = false;
		if (!bOnce)
		{
			FExternalProfiler* Profiler = FActiveExternalProfilerBase::GetActiveProfiler();
			if (Profiler)
			{
				Profiler->SetThreadName(ThreadIdToName(ThreadId));
			}
			bOnce = true;
		}
#endif

		TStatId StallStatId;
		bool bCountAsStall = false;
#if STATS
		TStatId StatName;
		FCycleCounter ProcessingTasks;
		if (ThreadId == ENamedThreads::GameThread)
		{
			StatName = GET_STATID(STAT_TaskGraph_GameTasks);
			StallStatId = GET_STATID(STAT_TaskGraph_GameStalls);
			bCountAsStall = true;
		}
		else if (ThreadId == ENamedThreads::GetRenderThread())
		{
			if (QueueIndex > 0)
			{
				StallStatId = GET_STATID(STAT_TaskGraph_RenderStalls);
				bCountAsStall = true;
			}
			// else StatName = none, we need to let the scope empty so that the render thread submits tasks in a timely manner. 
		}
		else
		{
			StatName = GET_STATID(STAT_TaskGraph_OtherTasks);
			StallStatId = GET_STATID(STAT_TaskGraph_OtherStalls);

			// Don't count RHI thread waits as stalls.
			bCountAsStall = ThreadId != ENamedThreads::RHIThread;
		}
		bool bTasksOpen = false;
		if (FThreadStats::IsCollectingData(StatName))
		{
			bTasksOpen = true;
			ProcessingTasks.Start(StatName);
		}
#endif
		const bool bIsRenderThreadMainQueue = (ENamedThreads::GetThreadIndex(ThreadId) == ENamedThreads::ActualRenderingThread) && (QueueIndex == 0);
		while (!Queue(QueueIndex).QuitForReturn)
		{
			const bool bIsRenderThreadAndPolling = bIsRenderThreadMainQueue && (GRenderThreadPollPeriodMs >= 0);
			const bool bStallQueueAllowStall = bAllowStall && !bIsRenderThreadAndPolling;
			FBaseGraphTask* Task = Queue(QueueIndex).StallQueue.Pop(0, bStallQueueAllowStall);
			TestRandomizedThreads();
			if (!Task)
			{
#if STATS
				if (bTasksOpen)
				{
					ProcessingTasks.Stop();
					bTasksOpen = false;
				}
#endif
				if (bAllowStall)
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(WaitForTasks);
					{
						FScopeCycleCounter Scope(StallStatId, EStatFlags::Verbose);
						Queue(QueueIndex).StallRestartEvent->Wait(bIsRenderThreadAndPolling ? GRenderThreadPollPeriodMs : MAX_uint32, bCountAsStall);
						if (Queue(QueueIndex).QuitForShutdown)
						{
							return ProcessedTasks;
						}
						TestRandomizedThreads();
					}
#if STATS
					if (!bTasksOpen && FThreadStats::IsCollectingData(StatName))
					{
						bTasksOpen = true;
						ProcessingTasks.Start(StatName);
					}
#endif
					continue;
				}
				else
				{
					break; // we were asked to quit
				}
			}
			else
			{
				Task->Execute(NewTasks, ENamedThreads::Type(ThreadId | (QueueIndex << ENamedThreads::QueueIndexShift)), true);
				ProcessedTasks++;
				TestRandomizedThreads();
			}
		}
#if STATS
		if (bTasksOpen)
		{
			ProcessingTasks.Stop();
			bTasksOpen = false;
		}
#endif
		return ProcessedTasks;
	}
	virtual void EnqueueFromThisThread(int32 QueueIndex, FBaseGraphTask* Task) override
	{
		checkThreadGraph(Task && Queue(QueueIndex).StallRestartEvent); // make sure we are started up
		uint32 PriIndex = ENamedThreads::GetTaskPriority(Task->GetThreadToExecuteOn()) ? 0 : 1;
		int32 ThreadToStart = Queue(QueueIndex).StallQueue.Push(Task, PriIndex);
		check(ThreadToStart < 0); // if I am stalled, then how can I be queueing a task?
	}

	virtual void RequestQuit(int32 QueueIndex) override
	{
		// this will not work under arbitrary circumstances. For example you should not attempt to stop threads unless they are known to be idle.
		if (!Queue(0).StallRestartEvent)
		{
			return;
		}
		if (QueueIndex == -1)
		{
			// we are shutting down
			checkThreadGraph(Queue(0).StallRestartEvent); // make sure we are started up
			checkThreadGraph(Queue(1).StallRestartEvent); // make sure we are started up
			Queue(0).QuitForShutdown = true;
			Queue(1).QuitForShutdown = true;
			Queue(0).StallRestartEvent->Trigger();
			Queue(1).StallRestartEvent->Trigger();
		}
		else
		{
			checkThreadGraph(Queue(QueueIndex).StallRestartEvent); // make sure we are started up
			Queue(QueueIndex).QuitForReturn = true;
		}
	}

	virtual bool EnqueueFromOtherThread(int32 QueueIndex, FBaseGraphTask* Task) override
	{
		TestRandomizedThreads();
		checkThreadGraph(Task && Queue(QueueIndex).StallRestartEvent); // make sure we are started up

		uint32 PriIndex = ENamedThreads::GetTaskPriority(Task->GetThreadToExecuteOn()) ? 0 : 1;
		int32 ThreadToStart = Queue(QueueIndex).StallQueue.Push(Task, PriIndex);

		if (ThreadToStart >= 0)
		{
			checkThreadGraph(ThreadToStart == 0);
			QUICK_SCOPE_CYCLE_COUNTER(STAT_TaskGraph_EnqueueFromOtherThread_Trigger);
			TASKGRAPH_SCOPE_CYCLE_COUNTER(1, STAT_TaskGraph_EnqueueFromOtherThread_Trigger);
			Queue(QueueIndex).StallRestartEvent->Trigger();
			return true;
		}
		return false;
	}

	virtual bool IsProcessingTasks(int32 QueueIndex) override
	{
		return !!Queue(QueueIndex).RecursionGuard;
	}

	virtual void WakeUp(int32 QueueIndex) override
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_TaskGraph_Wakeup_Trigger);
		TASKGRAPH_SCOPE_CYCLE_COUNTER(1, STAT_TaskGraph_Wakeup_Trigger);
		Queue(QueueIndex).StallRestartEvent->Trigger();
	}

private:

#if UE_EXTERNAL_PROFILING_ENABLED
	static inline const TCHAR* ThreadIdToName(ENamedThreads::Type ThreadId)
	{
		if (ThreadId == ENamedThreads::GameThread)
		{
			return TEXT("Game Thread");
		}
		else if (ThreadId == ENamedThreads::GetRenderThread())
		{
			return TEXT("Render Thread");
		}
		else if (ThreadId == ENamedThreads::RHIThread)
		{
			return TEXT("RHI Thread");
		}
		else
		{
			return TEXT("Unknown Named Thread");
		}
	}
#endif

	/** Grouping of the data for an individual queue. **/
	struct FThreadTaskQueue
	{
		FStallingTaskQueue<FBaseGraphTask, PLATFORM_CACHE_LINE_SIZE, 2> StallQueue;

		/** We need to disallow reentry of the processing loop **/
		uint32 RecursionGuard;

		/** Indicates we executed a return task, so break out of the processing loop. **/
		bool QuitForReturn;

		/** Indicates we executed a return task, so break out of the processing loop. **/
		bool QuitForShutdown;

		/** Event that this thread blocks on when it runs out of work. **/
		FEvent*	StallRestartEvent;

		FThreadTaskQueue()
			: RecursionGuard(0)
			, QuitForReturn(false)
			, QuitForShutdown(false)
			, StallRestartEvent(FPlatformProcess::GetSynchEventFromPool(false))
		{

		}
		~FThreadTaskQueue()
		{
			FPlatformProcess::ReturnSynchEventToPool(StallRestartEvent);
			StallRestartEvent = nullptr;
		}
	};

	FORCEINLINE FThreadTaskQueue& Queue(int32 QueueIndex)
	{
		checkThreadGraph(QueueIndex >= 0 && QueueIndex < ENamedThreads::NumQueues);
		return Queues[QueueIndex];
	}
	FORCEINLINE const FThreadTaskQueue& Queue(int32 QueueIndex) const
	{
		checkThreadGraph(QueueIndex >= 0 && QueueIndex < ENamedThreads::NumQueues);
		return Queues[QueueIndex];
	}

	FThreadTaskQueue Queues[ENamedThreads::NumQueues];
};

/**
*	FTaskThreadAnyThread
*	A class for managing a worker threads.
**/
class FTaskThreadAnyThread : public FTaskThreadBase
{
public:
	FTaskThreadAnyThread(int32 InPriorityIndex)
		: PriorityIndex(InPriorityIndex)
	{
	}
	virtual void ProcessTasksUntilQuit(int32 QueueIndex) override
	{
		if (PriorityIndex != (ENamedThreads::BackgroundThreadPriority >> ENamedThreads::ThreadPriorityShift))
		{
			FMemory::SetupTLSCachesOnCurrentThread();
		}
		check(!QueueIndex);
		const bool bIsMultiThread = FTaskGraphInterface::IsMultithread();
		do
		{
			ProcessTasks();			
		} while (!Queue.QuitForShutdown && bIsMultiThread); // @Hack - quit now when running with only one thread.
	}

	virtual uint64 ProcessTasksUntilIdle(int32 QueueIndex) override
	{
		if (FTaskGraphInterface::IsMultithread() == false)
		{
			return ProcessTasks();
		}
		else
		{
			check(0);
			return 0;
		}
	}

	// Calls meant to be called from any thread.

	/**
	*	Will cause the thread to return to the caller when it becomes idle. Used to return from ProcessTasksUntilQuit for named threads or to shut down unnamed threads.
	*	CAUTION: This will not work under arbitrary circumstances. For example you should not attempt to stop unnamed threads unless they are known to be idle.
	*	Return requests for named threads should be submitted from that named thread as FReturnGraphTask does.
	*	@param QueueIndex, Queue to request quit from
	**/
	virtual void RequestQuit(int32 QueueIndex) override
	{
		check(QueueIndex < 1);

		// this will not work under arbitrary circumstances. For example you should not attempt to stop threads unless they are known to be idle.
		checkThreadGraph(Queue.StallRestartEvent); // make sure we are started up
		Queue.QuitForShutdown = true;
		Queue.StallRestartEvent->Trigger();
	}

	virtual void WakeUp(int32 QueueIndex = 0) final override
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_TaskGraph_Wakeup_Trigger);
		TASKGRAPH_SCOPE_CYCLE_COUNTER(1, STAT_TaskGraph_Wakeup_Trigger);
		Queue.StallRestartEvent->Trigger();
	}

	void StallForTuning(bool Stall)
	{
		if (Stall)
		{
			Queue.StallForTuning.Lock();
			Queue.bStallForTuning = true;
		}
		else
		{
			Queue.bStallForTuning = false;
			Queue.StallForTuning.Unlock();
		}
	}
	/**
	*Return true if this thread is processing tasks. This is only a "guess" if you ask for a thread other than yourself because that can change before the function returns.
	*@param QueueIndex, Queue to request quit from
	**/
	virtual bool IsProcessingTasks(int32 QueueIndex) override
	{
		check(!QueueIndex);
		return !!Queue.RecursionGuard;
	}

#if UE_EXTERNAL_PROFILING_ENABLED
	virtual uint32 Run() override
	{
		static thread_local bool bOnce = false;
		if (!bOnce)
		{
			FExternalProfiler* Profiler = FActiveExternalProfilerBase::GetActiveProfiler();
			if (Profiler)
			{
				Profiler->SetThreadName(ThreadPriorityToName(PriorityIndex));
			}
			bOnce = true;
		}
		return FTaskThreadBase::Run();
	}
#endif

private:

#if UE_EXTERNAL_PROFILING_ENABLED
	static inline const TCHAR* ThreadPriorityToName(int32 PriorityIdx)
	{
		PriorityIdx <<= ENamedThreads::ThreadPriorityShift;
		if (PriorityIdx == ENamedThreads::HighThreadPriority)
		{
			return TEXT("Task Thread HP");
		}
		else if (PriorityIdx == ENamedThreads::NormalThreadPriority)
		{
			return TEXT("Task Thread NP");
		}
		else if (PriorityIdx == ENamedThreads::BackgroundThreadPriority)
		{
			return TEXT("Task Thread BP");
		}
		else
		{
			return TEXT("Task Thread Unknown Priority");
		}
	}
#endif

	/**
	*	Process tasks until idle. May block if bAllowStall is true
	*	@param QueueIndex, Queue to process tasks from
	*	@param bAllowStall,  if true, the thread will block on the stall event when it runs out of tasks.
	**/
	uint64 ProcessTasks()
	{
		TStatId StallStatId;
		bool bCountAsStall = true;
		uint64 ProcessedTasks = 0;
#if STATS
		TStatId StatName;
		FCycleCounter ProcessingTasks;
		StatName = GET_STATID(STAT_TaskGraph_OtherTasks);
		StallStatId = GET_STATID(STAT_TaskGraph_OtherStalls);
		bool bTasksOpen = false;
		if (FThreadStats::IsCollectingData(StatName))
		{
			bTasksOpen = true;
			ProcessingTasks.Start(StatName);
		}
#endif
		verify(++Queue.RecursionGuard == 1);
		bool bDidStall = false;
		while (1)
		{
			FBaseGraphTask* Task = FindWork();
			if (!Task)
			{
#if STATS
				if (bTasksOpen)
				{
					ProcessingTasks.Stop();
					bTasksOpen = false;
				}
#endif

				TestRandomizedThreads();
				const bool bIsMultithread = FTaskGraphInterface::IsMultithread();
				if (bIsMultithread)
				{
					FScopeCycleCounter Scope(StallStatId, EStatFlags::Verbose);
					Queue.StallRestartEvent->Wait(MAX_uint32, bCountAsStall);
					bDidStall = true;
				}
				if (Queue.QuitForShutdown || !bIsMultithread)
				{
					break;
				}
				TestRandomizedThreads();

#if STATS
				if (FThreadStats::IsCollectingData(StatName))
				{
					bTasksOpen = true;
					ProcessingTasks.Start(StatName);
				}
#endif
				continue;
			}
			TestRandomizedThreads();
#if YIELD_BETWEEN_TASKS
			// the Win scheduler is ill behaved and will sometimes let BG tasks run even when other tasks are ready....kick the scheduler between tasks
			if (!bDidStall && PriorityIndex == (ENamedThreads::BackgroundThreadPriority >> ENamedThreads::ThreadPriorityShift))
			{
				FPlatformProcess::Sleep(0);
			}
#endif
			bDidStall = false;
			Task->Execute(NewTasks, ENamedThreads::Type(ThreadId), true);
			ProcessedTasks++;
			TestRandomizedThreads();
			if (Queue.bStallForTuning)
			{
#if STATS
				if (bTasksOpen)
				{
					ProcessingTasks.Stop();
					bTasksOpen = false;
				}
#endif
				{
					FScopeLock Lock(&Queue.StallForTuning);
				}
#if STATS
				if (FThreadStats::IsCollectingData(StatName))
				{
					bTasksOpen = true;
					ProcessingTasks.Start(StatName);
				}
#endif
			}
		}
		verify(!--Queue.RecursionGuard);
		return ProcessedTasks;
	}

	/** Grouping of the data for an individual queue. **/
	struct FThreadTaskQueue
	{
		/** Event that this thread blocks on when it runs out of work. **/
		FEvent* StallRestartEvent;
		/** We need to disallow reentry of the processing loop **/
		uint32 RecursionGuard;
		/** Indicates we executed a return task, so break out of the processing loop. **/
		bool QuitForShutdown;
		/** Should we stall for tuning? **/
		bool bStallForTuning;
		FCriticalSection StallForTuning;

		FThreadTaskQueue()
			: StallRestartEvent(FPlatformProcess::GetSynchEventFromPool(false))
			, RecursionGuard(0)
			, QuitForShutdown(false)
			, bStallForTuning(false)
		{

		}
		~FThreadTaskQueue()
		{
			FPlatformProcess::ReturnSynchEventToPool(StallRestartEvent);
			StallRestartEvent = nullptr;
		}
	};

	/**
	*	Internal function to call the system looking for work. Called from this thread.
	*	@return New task to process.
	*/
	FBaseGraphTask* FindWork();

	/** Array of queues, only the first one is used for unnamed threads. **/
	FThreadTaskQueue Queue;

	int32 PriorityIndex;
};


/** 
	*	FWorkerThread
	*	Helper structure to aggregate a few items related to the individual threads.
**/
struct FWorkerThread
{
	/** The actual FTaskThread that manager this task **/
	FTaskThreadBase*	TaskGraphWorker;
	/** For internal threads, the is non-NULL and holds the information about the runable thread that was created. **/
	FRunnableThread*	RunnableThread;
	/** For external threads, this determines if they have been "attached" yet. Attachment is mostly setting up TLS for this individual thread. **/
	bool				bAttached;

	/** Constructor to set reasonable defaults. **/
	FWorkerThread()
		: TaskGraphWorker(nullptr)
		, RunnableThread(nullptr)
		, bAttached(false)
	{
	}
};

/**
*	FTaskGraphCompatibilityImplementation
*	Implementation of the centralized part of the task graph system using the ne low level Backend.
*	These parts of the system have no knowledge of the dependency graph, they exclusively work on tasks.
**/
class FTaskGraphCompatibilityImplementation final : public FTaskGraphInterface
{
	static thread_local TArray<FBaseGraphTask*> NewTasks;

	/** Array of callbacks to call before shutdown. **/
	TArray<TFunction<void()> > ShutdownCallbacks;

	/** Index of TLS slot for FWorkerThread* pointer. **/
	uint32				PerThreadIDTLSSlot;

	/** Number of named threads actually in use. **/
	int32				NumNamedThreads;
	int32				NumWorkerThreads;

	/** Individual foreground and background workers. **/
	int32				NumBackgroundWorkers;
	int32				NumForegroundWorkers;

	TArray<FWorkerThread> NamedThreads;

	FThreadSafeCounter	ReentrancyCheck;

	std::atomic<bool> bReserveWorkersEnabled{ false };

public:
	FTaskGraphCompatibilityImplementation(int32 InNumWorkerThreads) 
		: NumWorkerThreads(FForkProcessHelper::IsForkedMultithreadInstance() ? CVar_ForkedProcess_MaxWorkerThreads : InNumWorkerThreads)
	{
		TaskTrace::Init();

		if (FTaskGraphInterface::IsMultithread())
		{
			if (NumWorkerThreads <= 3)
			{
				GNumForegroundWorkers = 1;
			}

			NumBackgroundWorkers = FMath::Max(1, NumWorkerThreads - FMath::Min<int>(GNumForegroundWorkers, NumWorkerThreads));
			NumForegroundWorkers =  FMath::Max(1, NumWorkerThreads - NumBackgroundWorkers);

			LowLevelTasks::FScheduler::Get().StartWorkers(NumForegroundWorkers, NumBackgroundWorkers, FForkProcessHelper::IsForkedMultithreadInstance() ? FThread::Forkable : FThread::NonForkable, FPlatformAffinity::GetTaskThreadPriority(), FPlatformAffinity::GetTaskBPThreadPriority());

			NumNamedThreads = ENamedThreads::ActualRenderingThread + 1;
			ENamedThreads::bHasBackgroundThreads = 1;
			ENamedThreads::bHasHighPriorityThreads = 1;
		}
		else
		{
			LowLevelTasks::FScheduler::Get().StopWorkers();
			NumNamedThreads = ENamedThreads::ActualRenderingThread;
			ENamedThreads::bHasBackgroundThreads = 0;
			ENamedThreads::bHasHighPriorityThreads = 0;
		}	
		NamedThreads.AddDefaulted(NumNamedThreads);

		check(!ReentrancyCheck.GetValue()); // reentrant?
		ReentrancyCheck.Increment(); // just checking for reentrancy
		PerThreadIDTLSSlot = FPlatformTLS::AllocTlsSlot();

		for (int32 ThreadIndex = 0; ThreadIndex < NumNamedThreads; ThreadIndex++)
		{
			NamedThreads[ThreadIndex].TaskGraphWorker = new FNamedTaskThread;
			NamedThreads[ThreadIndex].TaskGraphWorker->Setup(ENamedThreads::Type(ThreadIndex), PerThreadIDTLSSlot, &NamedThreads[ThreadIndex]);
		}
	}

	~FTaskGraphCompatibilityImplementation() override
	{
		FCoreDelegates::TSConfigReadyForUse().RemoveAll(this);

		for (auto& Callback : ShutdownCallbacks)
		{
			Callback();
		}
		ShutdownCallbacks.Empty();
		for (int32 ThreadIndex = 0; ThreadIndex < NumNamedThreads; ThreadIndex++)
		{
			Thread(ThreadIndex).RequestQuit(-1);
			NamedThreads[ThreadIndex].bAttached = false;
		}
		LowLevelTasks::FScheduler::Get().StopWorkers();
		FPlatformTLS::FreeTlsSlot(PerThreadIDTLSSlot);
	}

	/** 
	*	Singleton returning the one and only FTaskGraphImplementation.
	*	Note that unlike most singletons, a manual call to FTaskGraphInterface::Startup is required before the singleton will return a valid reference.
	**/
	static FTaskGraphCompatibilityImplementation& Get()
	{
		checkThreadGraph(TaskGraphImplementationSingleton);
		return *static_cast<FTaskGraphCompatibilityImplementation*>(TaskGraphImplementationSingleton);
	}

	inline void* operator new(size_t Size)
	{
		return FMemory::Malloc(Size, 128u);
	}

	inline void operator delete(void* Ptr)
	{
		FMemory::Free(Ptr);
	}

	void SetTaskThreadPriorities(EThreadPriority Pri)
	{
		if (FTaskGraphInterface::IsMultithread())
		{
			NumBackgroundWorkers = FMath::Max(1, NumWorkerThreads - FMath::Min<int>(GNumForegroundWorkers, NumWorkerThreads));
			NumForegroundWorkers =  FMath::Max(1, NumWorkerThreads - NumBackgroundWorkers);

			LowLevelTasks::FScheduler::Get().StopWorkers();
			LowLevelTasks::FScheduler::Get().StartWorkers(NumForegroundWorkers, NumBackgroundWorkers, FForkProcessHelper::IsForkedMultithreadInstance() ? FThread::Forkable : FThread::NonForkable, Pri, FPlatformAffinity::GetTaskBPThreadPriority());
		}
	}

private:
	void QueueTask(class FBaseGraphTask* Task, bool bWakeUpWorker, ENamedThreads::Type InThreadToExecuteOn, ENamedThreads::Type InCurrentThreadIfKnown) override
	{
		check(ENamedThreads::GetThreadIndex(InThreadToExecuteOn) != ENamedThreads::AnyThread);

		ENamedThreads::Type CurrentThreadIfKnown;
		if (ENamedThreads::GetThreadIndex(InCurrentThreadIfKnown) == ENamedThreads::AnyThread)
		{
			CurrentThreadIfKnown = GetCurrentThread();
		}
		else
		{
			CurrentThreadIfKnown = ENamedThreads::GetThreadIndex(InCurrentThreadIfKnown);
			checkThreadGraph(CurrentThreadIfKnown == ENamedThreads::GetThreadIndex(GetCurrentThread()));
		}
		{
			int32 QueueToExecuteOn = ENamedThreads::GetQueueIndex(InThreadToExecuteOn);
			InThreadToExecuteOn = ENamedThreads::GetThreadIndex(InThreadToExecuteOn);
			FTaskThreadBase* Target = &Thread(InThreadToExecuteOn);
			if (InThreadToExecuteOn == ENamedThreads::GetThreadIndex(CurrentThreadIfKnown))
			{
				Target->EnqueueFromThisThread(QueueToExecuteOn, Task);
			}
			else
			{
				Target->EnqueueFromOtherThread(QueueToExecuteOn, Task);
			}
		}
	}

	int32 GetNumWorkerThreads() final override
	{
		return LowLevelTasks::FScheduler::Get().GetNumWorkers();
	}

	virtual	int32 GetNumForegroundThreads() final override
	{
		return NumForegroundWorkers;
	}

	virtual	int32 GetNumBackgroundThreads() final override
	{
		return NumBackgroundWorkers;
	}

	bool IsCurrentThreadKnown() final override
	{
		return FPlatformTLS::GetTlsValue(PerThreadIDTLSSlot) != nullptr || LowLevelTasks::FTask::GetActiveTask() != nullptr;
	}

	ENamedThreads::Type GetCurrentThreadIfKnown(bool bLocalQueue) final override
	{
		ENamedThreads::Type Result = GetCurrentThread();
		if (bLocalQueue && ENamedThreads::GetThreadIndex(Result) >= 0 && ENamedThreads::GetThreadIndex(Result) < NumNamedThreads)
		{
			Result = ENamedThreads::Type(int32(Result) | int32(ENamedThreads::LocalQueue));
		}
		return Result;
	}

	bool IsThreadProcessingTasks(ENamedThreads::Type ThreadToCheck) final override
	{
		int32 QueueIndex = ENamedThreads::GetQueueIndex(ThreadToCheck);
		ThreadToCheck = ENamedThreads::GetThreadIndex(ThreadToCheck);
		check(ThreadToCheck >= 0 && ThreadToCheck < NumNamedThreads);
		return Thread(ThreadToCheck).IsProcessingTasks(QueueIndex);
	}

	// External Thread API

	void AttachToThread(ENamedThreads::Type CurrentThread) final override
	{
		CurrentThread = ENamedThreads::GetThreadIndex(CurrentThread);
		check(CurrentThread >= 0 && CurrentThread < NumNamedThreads);
		check(!NamedThreads[CurrentThread].bAttached);
		Thread(CurrentThread).InitializeForCurrentThread();
	}

	uint64 ProcessThreadUntilIdle(ENamedThreads::Type CurrentThread) final override
	{
		int32 QueueIndex = ENamedThreads::GetQueueIndex(CurrentThread);
		CurrentThread = ENamedThreads::GetThreadIndex(CurrentThread);
		check(CurrentThread >= 0 && CurrentThread < NumNamedThreads);
		check(CurrentThread == GetCurrentThread());
		return Thread(CurrentThread).ProcessTasksUntilIdle(QueueIndex);
	}

	void ProcessThreadUntilRequestReturn(ENamedThreads::Type CurrentThread) final override
	{
		int32 QueueIndex = ENamedThreads::GetQueueIndex(CurrentThread);
		CurrentThread = ENamedThreads::GetThreadIndex(CurrentThread);
		check(CurrentThread >= 0 && CurrentThread < NumNamedThreads);
		check(CurrentThread == GetCurrentThread());
		Thread(CurrentThread).ProcessTasksUntilQuit(QueueIndex);
	}

	void RequestReturn(ENamedThreads::Type CurrentThread) final override
	{
		int32 QueueIndex = ENamedThreads::GetQueueIndex(CurrentThread);
		CurrentThread = ENamedThreads::GetThreadIndex(CurrentThread);
		check(CurrentThread != ENamedThreads::AnyThread);
		Thread(CurrentThread).RequestQuit(QueueIndex);
	}

	void WaitUntilTasksComplete(const FGraphEventArray& Tasks, ENamedThreads::Type CurrentThreadIfKnown = ENamedThreads::AnyThread) final override
	{
		TaskTrace::FWaitingScope WaitingScope(GetTraceIds(Tasks));
		TRACE_CPUPROFILER_EVENT_SCOPE(WaitUntilTasksComplete);

		// try retraction first and only invoke named thread waiting as a last resort
		const UE::FTimeout NeverTimeout = UE::FTimeout::Never();
		bool bAllTasksCompleted = true;
		for (const FGraphEventRef& Task : Tasks)
		{
			if (Task.IsValid())
			{
				Task->TryRetractAndExecute(NeverTimeout);

				bAllTasksCompleted &= Task->IsCompleted();
			}
		}

		if (bAllTasksCompleted)
		{
			return;
		}

		ENamedThreads::Type NamedThreadWithFlags = CurrentThreadIfKnown;
		if (ENamedThreads::GetThreadIndex(CurrentThreadIfKnown) == ENamedThreads::AnyThread)
		{
			bool bIsHiPri = !!ENamedThreads::GetTaskPriority(CurrentThreadIfKnown);
			int32 Priority = ENamedThreads::GetThreadPriorityIndex(CurrentThreadIfKnown);
			check(!ENamedThreads::GetQueueIndex(CurrentThreadIfKnown));
			CurrentThreadIfKnown = ENamedThreads::GetThreadIndex(GetCurrentThread());
			NamedThreadWithFlags = ENamedThreads::SetPriorities(CurrentThreadIfKnown, Priority, bIsHiPri);
		}
		else
		{
			CurrentThreadIfKnown = ENamedThreads::GetThreadIndex(CurrentThreadIfKnown);
			check(CurrentThreadIfKnown == ENamedThreads::GetThreadIndex(GetCurrentThread()));
		}

		WaitOnNamedThreadForTasks(Tasks, CurrentThreadIfKnown, NamedThreadWithFlags);
	}

	void WaitOnNamedThreadForTasks(const FGraphEventArray& Tasks, ENamedThreads::Type BaseNamedThread, ENamedThreads::Type NamedThreadWithFlags)
	{
		if (BaseNamedThread != ENamedThreads::AnyThread && BaseNamedThread < NumNamedThreads && !IsThreadProcessingTasks(NamedThreadWithFlags))
		{
			TGraphTask<FReturnGraphTask>::CreateTask(&Tasks, NamedThreadWithFlags).ConstructAndDispatchWhenReady(NamedThreadWithFlags);
			ProcessThreadUntilRequestReturn(NamedThreadWithFlags);
			return;
		}

		if (!FTaskGraphInterface::IsMultithread())
		{
			bool bAnyPending = false;
			for (int32 Index = 0; Index < Tasks.Num(); Index++)
			{
				FGraphEvent* Task = Tasks[Index].GetReference();
				if (Task && !Task->IsComplete())
				{
					bAnyPending = true;
					break;
				}
			}
			if (!bAnyPending)
			{
				return;
			}
			UE_LOGF(LogTaskGraph, Fatal, "Recursive waits are not allowed in single threaded mode.");
		}

		// We will just stall this thread on an event while we wait
		FScopedEvent Event;
		TriggerEventWhenTasksComplete(Event.Get(), Tasks, BaseNamedThread);
	}

	bool ProcessUntilTasksComplete(const FGraphEventArray& Tasks, ENamedThreads::Type CurrentThreadIfKnown, const FProcessTasksUpdateCallback& IdleWorkUpdate) final override
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ProcessUntilTasksComplete);

		ENamedThreads::Type NamedThreadWithFlags = CurrentThreadIfKnown;
		if (ENamedThreads::GetThreadIndex(CurrentThreadIfKnown) == ENamedThreads::AnyThread)
		{
			bool bIsHiPri = !!ENamedThreads::GetTaskPriority(CurrentThreadIfKnown);
			int32 Priority = ENamedThreads::GetThreadPriorityIndex(CurrentThreadIfKnown);
			check(!ENamedThreads::GetQueueIndex(CurrentThreadIfKnown));
			CurrentThreadIfKnown = ENamedThreads::GetThreadIndex(GetCurrentThread());
			NamedThreadWithFlags = ENamedThreads::SetPriorities(CurrentThreadIfKnown, Priority, bIsHiPri);
		}
		else
		{
			CurrentThreadIfKnown = ENamedThreads::GetThreadIndex(CurrentThreadIfKnown);
			check(CurrentThreadIfKnown == ENamedThreads::GetThreadIndex(GetCurrentThread()));
		}

		// Copy into faster array to avoid ref count changes, these will all be in Tasks array
		TArray<FGraphEvent*> RemainingTasks;
		RemainingTasks.SetNumUninitialized(Tasks.Num());
		for (int32 TaskIndex = 0; TaskIndex < Tasks.Num(); TaskIndex++)
		{
			RemainingTasks[TaskIndex] = Tasks[TaskIndex].GetReference();
			if (!ensure(RemainingTasks[TaskIndex] != nullptr))
			{
				// Fail if one of the events is not initialized
				UE_LOGF(LogTaskGraph, Error, "ProcessUntilTasksComplete was passed an invalid event!");
				return false;
			}
		}

		// Always start by processing named thread tasks once
		EProcessTasksOperation CurrentOperation = EProcessTasksOperation::ProcessNamedThreadTasks;
		
		const UE::FTimeout NeverTimeout = UE::FTimeout::Never();		
		bool bHasOtherTasks = true;

		while (CurrentOperation != EProcessTasksOperation::StopProcessing)
		{
			if (CurrentOperation == EProcessTasksOperation::ProcessNamedThreadTasks)
			{
				// Process until this named thread (could be a local queue) is idle, which should complete most of the tasks
				ProcessThreadUntilIdle(NamedThreadWithFlags);
			}
			else if (CurrentOperation == EProcessTasksOperation::ProcessAllOtherTasks || CurrentOperation == EProcessTasksOperation::ProcessOneOtherTask)
			{
				bHasOtherTasks = false;
				for (int32 TaskIndex = 0; TaskIndex < RemainingTasks.Num(); TaskIndex++)
				{
					FGraphEvent* Task = RemainingTasks[TaskIndex];
					// Don't process any named thread tasks here
					if (!Task->IsNamedThreadTask())
					{
						bHasOtherTasks = true;
						if (Task->TryRetractAndExecute(NeverTimeout) && CurrentOperation == EProcessTasksOperation::ProcessOneOtherTask)
						{
							break; // Completed a task, check logic again
						}
					}
				}
			}
			else if (CurrentOperation == EProcessTasksOperation::WaitUntilComplete)
			{
				FGraphEventArray TasksToWaitOn;
				for (FGraphEvent* TaskEvent : RemainingTasks)
				{
					TasksToWaitOn.Add(TaskEvent->GetCompletionEvent());
				}

				TaskTrace::FWaitingScope WaitingScope(GetTraceIds(TasksToWaitOn));
				WaitOnNamedThreadForTasks(TasksToWaitOn, CurrentThreadIfKnown, NamedThreadWithFlags);

				// All tasks must have succeeded
				return true;
			}

			// Update remaining tasks list
			for (int32 TaskIndex = RemainingTasks.Num() - 1; TaskIndex >= 0; TaskIndex--)
			{
				if (RemainingTasks[TaskIndex]->IsCompleted())
				{
					RemainingTasks.RemoveAtSwap(TaskIndex, EAllowShrinking::No);
				}
			}

			if (RemainingTasks.Num() == 0)
			{
				// All complete
				return true;
			}

			// Run callback function if it exists, otherwise default to ProcessAllOtherTasks
			if (IdleWorkUpdate)
			{
				CurrentOperation = IdleWorkUpdate(RemainingTasks.Num());
				check(CurrentOperation >= EProcessTasksOperation::ProcessAllOtherTasks && CurrentOperation <= EProcessTasksOperation::StopProcessing);
			}
			else
			{
				CurrentOperation = EProcessTasksOperation::ProcessAllOtherTasks;
			}

			if ((CurrentOperation == EProcessTasksOperation::ProcessAllOtherTasks || CurrentOperation == EProcessTasksOperation::ProcessOneOtherTask)
				&& !bHasOtherTasks)
			{
				// If we have no more tasks to try and retract, just wait which will handle processing named thread tasks as well
				CurrentOperation = EProcessTasksOperation::WaitUntilComplete;
			}
		}

		// Stopped due to callback, some tasks may not be complete
		return false;
	}

	void TriggerEventWhenTasksComplete(FEvent* InEvent, const FGraphEventArray& Tasks, ENamedThreads::Type CurrentThreadIfKnown = ENamedThreads::AnyThread, ENamedThreads::Type TriggerThread = ENamedThreads::AnyHiPriThreadHiPriTask) final override
	{
		check(InEvent);
		bool bAnyPending = true;
		if (Tasks.Num() < 8) // don't bother to check for completion if there are lots of prereqs...too expensive to check
		{
			bAnyPending = false;
			for (int32 Index = 0; Index < Tasks.Num(); Index++)
			{
				FGraphEvent* Task = Tasks[Index].GetReference();
				if (Task && !Task->IsComplete())
				{
					bAnyPending = true;
					break;
				}
			}
		}
		if (!bAnyPending)
		{
			TestRandomizedThreads();
			InEvent->Trigger();
			return;
		}

		// With the new taskgraph frontend FGraphEventArray can be used as prerequisites.
		// This which avoids a potential deadlock situation if all threads are used
		// while the condition is met because this one will execute inline from the thread that completed the prereqs
		// instead of scheduling yet another task just to trigger the event that may never run if the taskgraph is already full.
		UE::Tasks::Launch(
			TEXT("TriggerEventWhenTaskComplete"),
			[InEvent]()
			{
				InEvent->Trigger();
			},
			Tasks,
			LowLevelTasks::ETaskPriority::Normal,
			UE::Tasks::EExtendedTaskPriority::Inline
		);
	}

	void AddShutdownCallback(TFunction<void()>& Callback) override
	{
		ShutdownCallbacks.Emplace(Callback);
	}

	void WakeNamedThread(ENamedThreads::Type ThreadToWake) override
	{
		const ENamedThreads::Type ThreadIndex = ENamedThreads::GetThreadIndex(ThreadToWake);
		if (ThreadIndex < NumNamedThreads)
		{
			Thread(ThreadIndex).WakeUp(ENamedThreads::GetQueueIndex(ThreadToWake));
		}
	}
	
	virtual FBaseGraphTask* FindWork(ENamedThreads::Type ThreadInNeed)
	{
		check(false);
		return nullptr;
	}

	void StallForTuning(int32 Index, bool Stall) override
	{
	}

	/** 
	*	Examines the TLS to determine the identity of the current thread.
	*	@return	Id of the thread that is this thread or ENamedThreads::AnyThread if this thread is unknown or is a named thread that has not attached yet.
	**/
	ENamedThreads::Type GetCurrentThread()
	{
		ENamedThreads::Type CurrentThreadIfKnown = ENamedThreads::AnyThread;
		FWorkerThread* TLSPointer = (FWorkerThread*)FPlatformTLS::GetTlsValue(PerThreadIDTLSSlot);
		if (TLSPointer)
		{
			checkThreadGraph(TLSPointer - NamedThreads.GetData() >= 0 && TLSPointer - NamedThreads.GetData() < NamedThreads.Num());
			int32 ThreadIndex = UE_PTRDIFF_TO_INT32(TLSPointer - NamedThreads.GetData());
			checkThreadGraph(Thread(ThreadIndex).GetThreadId() == ThreadIndex);
			if (ThreadIndex < NumNamedThreads)
			{
				CurrentThreadIfKnown = ENamedThreads::Type(ThreadIndex);
			}
		}
		else 
		{
			const LowLevelTasks::FTask* Task = LowLevelTasks::FTask::GetActiveTask();
			if (Task != nullptr)
			{
				ENamedThreads::Type ThreadConversion[int(LowLevelTasks::ETaskPriority::Count)] = { ENamedThreads::HighThreadPriority, ENamedThreads::NormalThreadPriority, ENamedThreads::BackgroundThreadPriority, ENamedThreads::BackgroundThreadPriority, ENamedThreads::BackgroundThreadPriority };
				ENamedThreads::Type TaskConversion[int(LowLevelTasks::ETaskPriority::Count)] = { ENamedThreads::NormalTaskPriority, ENamedThreads::NormalTaskPriority, ENamedThreads::HighTaskPriority, ENamedThreads::NormalTaskPriority, ENamedThreads::NormalTaskPriority };
				CurrentThreadIfKnown = (ENamedThreads::Type)(ENamedThreads::AnyThread | ThreadConversion[int(Task->GetPriority())] | TaskConversion[int(Task->GetPriority())]);
			}
		}
		return CurrentThreadIfKnown;
	}

	/** 
	*	Internal function to verify an index and return the corresponding FTaskThread
	*	@param	Index; Id of the thread to retrieve.
	*	@return	Reference to the corresponding thread.
	**/
	inline FTaskThreadBase& Thread(int32 Index)
	{
		checkThreadGraph(NamedThreads[Index].TaskGraphWorker->GetThreadId() == Index);
		return *NamedThreads[Index].TaskGraphWorker;
	}
};

thread_local TArray<FBaseGraphTask*> FTaskGraphCompatibilityImplementation::NewTasks;

// Implementations of FTaskThread function that require knowledge of FTaskGraphImplementation

FBaseGraphTask* FTaskThreadAnyThread::FindWork()
{
	return TaskGraphImplementationSingleton->FindWork(ThreadId);
}


// Statics in FTaskGraphInterface

void FTaskGraphInterface::Startup(int32 NumThreads)
{
	// Limit the total number of threads used
#if defined(UE_TASKGRAPH_THREAD_LIMIT)
	NumThreads = FMath::Min(NumThreads, int32(UE_TASKGRAPH_THREAD_LIMIT));
#endif

	//We want to reduce the number of overall threads that UE uses so that there is are some 
	//free cores available for other things like the Browser or other Applications. 
	//Therefore we increase the number of Foreground workers, which are mostly unused. 
	//But when HighPrio work comes in the Foreground workers will be available and get the job done.
	bool bIsCookCommandlet = FParse::Param(FCommandLine::Get(), TEXT("cookcommandlet")) || FParse::Param(FCommandLine::Get(), TEXT("run=cook"));
	if (!bIsCookCommandlet)
	{
		GNumForegroundWorkers = FMath::Max(FMath::DivideAndRoundUp(NumThreads, 21), 2);
	}

	FParse::Value(FCommandLine::Get(), TEXT("-foregroundworkers="), GNumForegroundWorkers);
	FParse::Value(FCommandLine::Get(), TEXT("-oversubscriptionratio="), GTaskGraphOversubscriptionRatio);

	TaskGraphImplementationSingleton = new FTaskGraphCompatibilityImplementation(NumThreads);
}

void FTaskGraphInterface::Shutdown()
{
	delete TaskGraphImplementationSingleton;
	TaskGraphImplementationSingleton = nullptr;
}

bool FTaskGraphInterface::IsRunning()
{
    return TaskGraphImplementationSingleton != NULL;
}

FTaskGraphInterface& FTaskGraphInterface::Get()
{
	checkThreadGraph(TaskGraphImplementationSingleton);
	return *TaskGraphImplementationSingleton;
}

bool FTaskGraphInterface::IsMultithread()
{
	return FPlatformProcess::SupportsMultithreading() || (FForkProcessHelper::IsForkedMultithreadInstance() && GAllowTaskGraphForkMultithreading);
}

FGraphEventImplAllocator& GetGraphEventImplAllocator()
{
	static FGraphEventImplAllocator Singleton;
	return Singleton;
}


DECLARE_CYCLE_STAT(TEXT("FBroadcastTask"), STAT_FBroadcastTask, STATGROUP_TaskGraphTasks);

static int32 GPrintBroadcastWarnings = false;

static FAutoConsoleVariableRef CVarPrintBroadcastWarnings(
	TEXT("TaskGraph.PrintBroadcastWarnings"),
	GPrintBroadcastWarnings,
	TEXT("If > 0 taskgraph will emit warnings when waiting on broadcasts"),
	ECVF_Default
);

class FBroadcastTask
{
public:
	FBroadcastTask(TFunction<void(ENamedThreads::Type CurrentThread)>& InFunction, double InStartTime, const TCHAR* InName, ENamedThreads::Type InDesiredThread, FThreadSafeCounter* InStallForTaskThread, FEvent* InTaskEvent, FEvent* InCallerEvent)
		: Function(InFunction)
		, DesiredThread(InDesiredThread)
		, StallForTaskThread(InStallForTaskThread)
		, TaskEvent(InTaskEvent)
		, CallerEvent(InCallerEvent)
		, StartTime(InStartTime)
		, Name(InName)
	{
	}
	ENamedThreads::Type GetDesiredThread()
	{
		return DesiredThread;
	}

	TStatId GetStatId() const
	{
		return GET_STATID(STAT_FBroadcastTask);
	}

	static ESubsequentsMode::Type GetSubsequentsMode() { return ESubsequentsMode::TrackSubsequents; }
	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
	{
		auto LogWarningIfSlow = [this](const TCHAR* Msg)
		{
			const bool bNamedThread = TaskEvent == nullptr; // we don't wait for named threads broadcasting, as they can be quite busy, 
			// it may take longer to reach them. do not report slow processing in this case
			const double ThisTime = FPlatformTime::Seconds() - StartTime;
			if (!bNamedThread && ThisTime > 0.02)
			{
				UE_CLOGF(GPrintBroadcastWarnings, LogTaskGraph, Warning, "Task graph took %6.2fms for %ls to %ls", ThisTime * 1000.0, Name, Msg);
			}
		};

		LogWarningIfSlow(TEXT("receive broadcast."));

		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_Broadcast_PayloadFunction);
			Function(CurrentThread);
		}

		LogWarningIfSlow(TEXT("receive broadcast and do processing."));

		if (StallForTaskThread)
		{
			if (StallForTaskThread->Decrement())
			{
				if (TaskEvent)
				{
					QUICK_SCOPE_CYCLE_COUNTER(STAT_Broadcast_WaitForOthers);
					TaskEvent->Wait();
					LogWarningIfSlow(TEXT("receive broadcast do processing and wait for other task threads."));
				}
			}
			else
			{
				CallerEvent->Trigger();
				LogWarningIfSlow(TEXT("to receive broadcast do processing and trigger other task threads."));
			}
		}
	}
private:
	TFunction<void(ENamedThreads::Type CurrentThread)> Function;
	const ENamedThreads::Type DesiredThread;
	FThreadSafeCounter* StallForTaskThread;
	FEvent* TaskEvent;
	FEvent* CallerEvent;
	double StartTime;
	const TCHAR* Name;
};

void FTaskGraphInterface::BroadcastSlow_OnlyUseForSpecialPurposes(bool bDoTaskThreads, bool bDoBackgroundThreads, TFunction<void(ENamedThreads::Type CurrentThread)>& Callback)
{
	double StartTime = FPlatformTime::Seconds();

	QUICK_SCOPE_CYCLE_COUNTER(STAT_FTaskGraphInterface_BroadcastSlow_OnlyUseForSpecialPurposes);
	TRACE_CPUPROFILER_EVENT_SCOPE(FTaskGraphInterface_BroadcastSlow);
	check(FPlatformTLS::GetCurrentThreadId() == GGameThreadId);

	Callback(ENamedThreads::GameThread_Local);

	if (!TaskGraphImplementationSingleton)
	{
		// we aren't going yet
		return;
	}


	TArray<FEvent*> TaskEvents;

	FEvent* MyEvent = nullptr;
	FGraphEventArray TaskThreadTasks;
	FThreadSafeCounter StallForTaskThread;
	if (bDoTaskThreads)
	{
		MyEvent = FPlatformProcess::GetSynchEventFromPool(false);

		int32 Workers = bDoBackgroundThreads ? TaskGraphImplementationSingleton->GetNumWorkerThreads() : GNumForegroundWorkers;
		StallForTaskThread.Add(Workers);

		TaskEvents.Reserve(StallForTaskThread.GetValue());
		{

			for (int32 Index = 0; Index < Workers; Index++)
			{
				FEvent* TaskEvent = FPlatformProcess::GetSynchEventFromPool(false);
				TaskEvents.Add(TaskEvent);
				TaskThreadTasks.Add(TGraphTask<FBroadcastTask>::CreateTask().ConstructAndDispatchWhenReady(Callback, StartTime, TEXT("NPTask"), ENamedThreads::AnyHiPriThreadHiPriTask, &StallForTaskThread, TaskEvent, MyEvent));
			}

		}

		check(TaskGraphImplementationSingleton);
	}


	if (bDoTaskThreads)
	{
		check(MyEvent);
		if (MyEvent && !MyEvent->Wait(3000))
		{
			UE_LOGF(LogTaskGraph, Log, "FTaskGraphInterface::BroadcastSlow_OnlyUseForSpecialPurposes Broadcast failed after three seconds. Ok during automated tests.");
		}
		for (FEvent* TaskEvent : TaskEvents)
		{
			TaskEvent->Trigger();
		}
		{
			const double StartTimeInner = FPlatformTime::Seconds();
			QUICK_SCOPE_CYCLE_COUNTER(STAT_Broadcast_WaitForTaskThreads);
			TRACE_CPUPROFILER_EVENT_SCOPE(Broadcast_WaitForTaskThreads);
			FTaskGraphInterface::Get().WaitUntilTasksComplete(TaskThreadTasks, ENamedThreads::GameThread_Local);
			{
				const double ThisTime = FPlatformTime::Seconds() - StartTimeInner;
				if (ThisTime > 0.02)
				{
					UE_CLOGF(GPrintBroadcastWarnings, LogTaskGraph, Warning, "Task graph took %6.2fms to wait for task thread broadcast.", ThisTime * 1000.0);
				}
			}
		}
	}

	if (IsRHIThreadRunning())
	{
		TGraphTask<FBroadcastTask>::CreateTask().ConstructAndDispatchWhenReady(Callback, StartTime, TEXT("RHIT"), ENamedThreads::SetTaskPriority(ENamedThreads::RHIThread, ENamedThreads::HighTaskPriority), nullptr, nullptr, nullptr);
	}
	ENamedThreads::Type RenderThread = ENamedThreads::GetRenderThread();
	if (RenderThread != ENamedThreads::GameThread)
	{
		TGraphTask<FBroadcastTask>::CreateTask().ConstructAndDispatchWhenReady(Callback, StartTime, TEXT("RT"), ENamedThreads::SetTaskPriority(RenderThread, ENamedThreads::HighTaskPriority), nullptr, nullptr, nullptr);
	}

	for (FEvent* TaskEvent : TaskEvents)
	{
		FPlatformProcess::ReturnSynchEventToPool(TaskEvent);
	}
	if (MyEvent)
	{
		FPlatformProcess::ReturnSynchEventToPool(MyEvent);
	}
	{
		const double ThisTime = FPlatformTime::Seconds() - StartTime;
		if (ThisTime > 0.02)
		{
			UE_CLOGF(GPrintBroadcastWarnings, LogTaskGraph, Warning, "Task graph took %6.2fms to broadcast.", ThisTime * 1000.0);
		}
	}
}

static void HandleNumWorkerThreadsToIgnore(const TArray<FString>& Args)
{
	if (Args.Num() > 0)
	{
		int32 Arg = FCString::Atoi(*Args[0]);
		int32 MaxNumPerBank = FTaskGraphInterface::Get().GetNumWorkerThreads() + GNumWorkerThreadsToIgnore;
		if (Arg < MaxNumPerBank && Arg >= 0 && Arg != GNumWorkerThreadsToIgnore)
		{
			if (Arg > GNumWorkerThreadsToIgnore)
			{
				for (int32 Index = MaxNumPerBank - GNumWorkerThreadsToIgnore - 1; Index >= MaxNumPerBank - Arg; Index--)
				{
					TaskGraphImplementationSingleton->StallForTuning(Index, true);
				}
			}
			else
			{
				for (int32 Index = MaxNumPerBank - Arg - 1; Index >= MaxNumPerBank - GNumWorkerThreadsToIgnore; Index--)
				{
					TaskGraphImplementationSingleton->StallForTuning(Index, false);
				}
			}
			GNumWorkerThreadsToIgnore = Arg;
		}
	}
	UE_LOGF(LogConsoleResponse, Display, "Currently ignoring %d threads per priority bank", GNumWorkerThreadsToIgnore);
}

static FAutoConsoleCommand CVarNumWorkerThreadsToIgnore(
	TEXT("TaskGraph.NumWorkerThreadsToIgnore"),
	TEXT("Used to tune the number of task threads. Generally once you have found the right value, PlatformMisc::NumberOfWorkerThreadsToSpawn() should be hardcoded."),
	FConsoleCommandWithArgsDelegate::CreateStatic(&HandleNumWorkerThreadsToIgnore)
	);

static void SetTaskThreadPriority(const TArray<FString>& Args)
{
	EThreadPriority Pri = TPri_Normal;
	if (Args.Num() && Args[0] == TEXT("abovenormal"))
	{
		Pri = TPri_AboveNormal;
		UE_LOGF(LogConsoleResponse, Display, "Setting task thread priority to above normal.");
	}
	else if (Args.Num() && Args[0] == TEXT("belownormal"))
	{
		Pri = TPri_BelowNormal;
		UE_LOGF(LogConsoleResponse, Display, "Setting task thread priority to below normal.");
	}
	else
	{
		UE_LOGF(LogConsoleResponse, Display, "Setting task thread priority to normal.");
	}

	FTaskGraphCompatibilityImplementation::Get().SetTaskThreadPriorities(Pri);
}

static FAutoConsoleCommand TaskThreadPriorityCmd(
	TEXT("TaskGraph.TaskThreadPriority"),
	TEXT("Sets the priority of the task threads. Argument is one of belownormal, normal or abovenormal."),
	FConsoleCommandWithArgsDelegate::CreateStatic(&SetTaskThreadPriority)
	);

/////////////////////////////////////////////////////////////
// "any task" support. these functions allocate excessively (per input task plus more)
// can be reduced to a single alloc if this is a perf issue

int32 WaitForAnyTaskCompleted(const FGraphEventArray& GraphEvents, FTimespan Timeout /*= FTimespan::MaxValue()*/)
{
	return UE::Tasks::WaitAny(GraphEvents, Timeout);
}

FGraphEventRef AnyTaskCompleted(const FGraphEventArray& GraphEvents)
{
	if (UNLIKELY(GraphEvents.Num() == 0))
	{
		FGraphEventRef Result = FGraphEvent::CreateGraphEvent();
		Result->DispatchSubsequents();
		return Result;
	}

	struct FSharedData
	{
		explicit FSharedData(uint32 InitRefCount)
			: RefCount(InitRefCount)
		{
		}

		FGraphEventRef Event = FGraphEvent::CreateGraphEvent();
		std::atomic<uint32> RefCount;
	};

	FSharedData* SharedData = new FSharedData(GraphEvents.Num());
	// `SharedData` can be destroyed before leaving the scope, cache the result locally
	FGraphEventRef Result = SharedData->Event;

	for (const FGraphEventRef& GraphEvent : GraphEvents)
	{
		FFunctionGraphTask::CreateAndDispatchWhenReady(
			[SharedData, Num = GraphEvents.Num()]
			{
				// cache the local copy as `SharedData` can be concurrently deleted right after decrementing the ref counter
				FGraphEventRef Event = SharedData->Event;
				uint32 PrevRefCount = SharedData->RefCount.fetch_sub(1, std::memory_order_acq_rel); // acq_rel to sync between tasks
				
				if (UNLIKELY(PrevRefCount == Num))
				{	// the first completed task
					Event->DispatchSubsequents();
				}
				else if (UNLIKELY(PrevRefCount == 1))
				{	// the last competed task
					delete SharedData;
				}
			},
			TStatId{},
			GraphEvent
		);
	}

	return Result;
}