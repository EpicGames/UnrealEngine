// Copyright Epic Games, Inc. All Rights Reserved.

#include "Async/TaskTrace.h"

#if UE_TASK_TRACE_ENABLED

#include "CoreMinimal.h"
#include "Trace/Trace.h"
#include "Trace/Trace.inl"
#include "HAL/PlatformTime.h"

namespace TaskTrace
{
	UE_TRACE_CHANNEL_DEFINE(TaskChannel, "Task system lifecycle, scheduling, and dependencies. Data can be viewed in the Timing Profiler (Task Graph Insights).");

	UE_TRACE_EVENT_BEGIN(TaskTrace, Init)
		UE_TRACE_EVENT_FIELD(uint32, Version)
	UE_TRACE_EVENT_END()

	UE_TRACE_EVENT_BEGIN(TaskTrace, Created)
		UE_TRACE_EVENT_FIELD(uint64, Timestamp)
		UE_TRACE_EVENT_FIELD(uint64, TaskId)
		UE_TRACE_EVENT_FIELD(uint64, TaskSize)
	UE_TRACE_EVENT_END()

	UE_TRACE_EVENT_BEGIN(TaskTrace, Launched)
		UE_TRACE_EVENT_FIELD(uint64, Timestamp)
		UE_TRACE_EVENT_FIELD(uint64, TaskId)
		UE_TRACE_EVENT_FIELD(UE::Trace::WideString, DebugName)
		UE_TRACE_EVENT_FIELD(bool, Tracked)
		UE_TRACE_EVENT_FIELD(int32, ThreadToExecuteOn)
		UE_TRACE_EVENT_FIELD(uint64, TaskSize)
	UE_TRACE_EVENT_END()

	UE_TRACE_EVENT_BEGIN(TaskTrace, Scheduled)
		UE_TRACE_EVENT_FIELD(uint64, Timestamp)
		UE_TRACE_EVENT_FIELD(uint64, TaskId)
	UE_TRACE_EVENT_END()

	UE_TRACE_EVENT_BEGIN(TaskTrace, SubsequentAdded)
		UE_TRACE_EVENT_FIELD(uint64, Timestamp)
		UE_TRACE_EVENT_FIELD(uint64, TaskId)
		UE_TRACE_EVENT_FIELD(uint64, SubsequentId)
	UE_TRACE_EVENT_END()

	UE_TRACE_EVENT_BEGIN(TaskTrace, Started)
		UE_TRACE_EVENT_FIELD(uint64, Timestamp)
		UE_TRACE_EVENT_FIELD(uint64, TaskId)
	UE_TRACE_EVENT_END()

	UE_TRACE_EVENT_BEGIN(TaskTrace, Finished)
		UE_TRACE_EVENT_FIELD(uint64, Timestamp)
		UE_TRACE_EVENT_FIELD(uint64, TaskId)
	UE_TRACE_EVENT_END()

	UE_TRACE_EVENT_BEGIN(TaskTrace, Completed)
		UE_TRACE_EVENT_FIELD(uint64, Timestamp)
		UE_TRACE_EVENT_FIELD(uint64, TaskId)
	UE_TRACE_EVENT_END()

	UE_TRACE_EVENT_BEGIN(TaskTrace, Destroyed)
		UE_TRACE_EVENT_FIELD(uint64, Timestamp)
		UE_TRACE_EVENT_FIELD(uint64, TaskId)
	UE_TRACE_EVENT_END()

	UE_TRACE_EVENT_BEGIN(TaskTrace, WaitingStarted)
		UE_TRACE_EVENT_FIELD(uint64, Timestamp)
		UE_TRACE_EVENT_FIELD(uint64[], Tasks)
	UE_TRACE_EVENT_END()

	UE_TRACE_EVENT_BEGIN(TaskTrace, WaitingFinished)
		UE_TRACE_EVENT_FIELD(uint64, Timestamp)
	UE_TRACE_EVENT_END()

	UE_AUTORTFM_ALWAYS_OPEN
	FId GenerateTaskId()
	{
		static std::atomic<FId> UId{ 0 };
		FId Id = UId.fetch_add(1, std::memory_order_relaxed);
		checkf(Id != InvalidId, TEXT("TraceId overflow"));
		return Id;
	}

	static bool bGTaskTraceInitialized = false;
	std::atomic<uint32> ExecuteTaskSpecId = 0;

	void Init()
	{
		UE_TRACE_LOG(TaskTrace, Init, TaskChannel)
			<< Init.Version(0);

		bGTaskTraceInitialized = true;
#if CPUPROFILERTRACE_ENABLED
		if (UE_TRACE_CHANNELEXPR_IS_ENABLED(CpuChannel))
		{
			ExecuteTaskSpecId = FCpuProfilerTrace::OutputEventType("ExecuteTask", __FILE__, __LINE__);
		}
#endif
	}

	void Created(FId TaskId, uint64 TaskSize)
	{
		if (UE_TRACE_CHANNELEXPR_IS_ENABLED(TaskChannel))
		{
			if (!bGTaskTraceInitialized)
			{
				return;
			}

			check(TaskId != InvalidId);

			UE_TRACE_LOG(TaskTrace, Created, TaskChannel)
				<< Created.Timestamp(FPlatformTime::Cycles64())
				<< Created.TaskId(TaskId)
				<< Created.TaskSize(TaskSize);
		}
	}

	void Launched(FId TaskId, const TCHAR* DebugName, bool bTracked, ENamedThreads::Type ThreadToExecuteOn, uint64 TaskSize)
	{
		if (UE_TRACE_CHANNELEXPR_IS_ENABLED(TaskChannel))
		{
			if (!bGTaskTraceInitialized)
			{
				return;
			}

			check(TaskId != InvalidId);

			UE_TRACE_LOG(TaskTrace, Launched, TaskChannel)
				<< Launched.Timestamp(FPlatformTime::Cycles64())
				<< Launched.TaskId(TaskId)
				<< Launched.DebugName(DebugName != nullptr ? DebugName : TEXT(""))
				<< Launched.Tracked(bTracked)
				<< Launched.ThreadToExecuteOn(ThreadToExecuteOn)
				<< Launched.TaskSize(TaskSize);
		}
	}

	void Scheduled(FId TaskId)
	{
		if (UE_TRACE_CHANNELEXPR_IS_ENABLED(TaskChannel))
		{
			if (!bGTaskTraceInitialized)
			{
				return;
			}

			check(TaskId != InvalidId);

			UE_TRACE_LOG(TaskTrace, Scheduled, TaskChannel)
				<< Scheduled.Timestamp(FPlatformTime::Cycles64())
				<< Scheduled.TaskId(TaskId);
		}
	}

	void SubsequentAdded(FId TaskId, FId SubsequentId)
	{
		if (UE_TRACE_CHANNELEXPR_IS_ENABLED(TaskChannel))
		{
			if (!bGTaskTraceInitialized)
			{
				return;
			}

			// "empty" FGraphEvent is used for synchronisation only, to wait for a notification. It doesn't have an associated task and ID.
			if (TaskId == InvalidId)
			{
				TaskId = GenerateTaskId();
			}

			check(SubsequentId != InvalidId);
			UE_TRACE_LOG(TaskTrace, SubsequentAdded, TaskChannel)
				<< SubsequentAdded.Timestamp(FPlatformTime::Cycles64())
				<< SubsequentAdded.TaskId(TaskId)
				<< SubsequentAdded.SubsequentId(SubsequentId);
		}
	}

	void Started(FId TaskId)
	{
		if (UE_TRACE_CHANNELEXPR_IS_ENABLED(TaskChannel))
		{
			if (!bGTaskTraceInitialized)
			{
				return;
			}

			check(TaskId != InvalidId);

			UE_TRACE_LOG(TaskTrace, Started, TaskChannel)
				<< Started.Timestamp(FPlatformTime::Cycles64())
				<< Started.TaskId(TaskId);
		}
	}

	void Finished(FId TaskId)
	{
		if (UE_TRACE_CHANNELEXPR_IS_ENABLED(TaskChannel))
		{
			if (!bGTaskTraceInitialized)
			{
				return;
			}

			check(TaskId != InvalidId);

			UE_TRACE_LOG(TaskTrace, Finished, TaskChannel)
				<< Finished.Timestamp(FPlatformTime::Cycles64())
				<< Finished.TaskId(TaskId);
		}
	}

	void Completed(FId TaskId)
	{
		if (UE_TRACE_CHANNELEXPR_IS_ENABLED(TaskChannel))
		{
			if (!bGTaskTraceInitialized)
			{
				return;
			}

			// "empty" FGraphEvent is used for synchronisation only, to wait for a notification. It doesn't have an associated task and ID.
			if (TaskId == InvalidId)
			{
				TaskId = GenerateTaskId();
			}

			UE_TRACE_LOG(TaskTrace, Completed, TaskChannel)
				<< Completed.Timestamp(FPlatformTime::Cycles64())
				<< Completed.TaskId(TaskId);
		}
	}

	void Destroyed(FId TaskId)
	{
		if (UE_TRACE_CHANNELEXPR_IS_ENABLED(TaskChannel))
		{
			if (!bGTaskTraceInitialized)
			{
				return;
			}

			UE_TRACE_LOG(TaskTrace, Destroyed, TaskChannel)
				<< Destroyed.Timestamp(FPlatformTime::Cycles64())
				<< Destroyed.TaskId(TaskId);
		}
	}

	FWaitingScope::FWaitingScope(const TArray<FId>& Tasks)
	{
		if (UE_TRACE_CHANNELEXPR_IS_ENABLED(TaskChannel))
		{
			if (!bGTaskTraceInitialized)
			{
				return;
			}

			UE_TRACE_LOG(TaskTrace, WaitingStarted, TaskChannel)
				<< WaitingStarted.Timestamp(FPlatformTime::Cycles64())
				<< WaitingStarted.Tasks(Tasks.GetData(), Tasks.Num());
		}
	}

	FWaitingScope::FWaitingScope(FId TaskId)
	{
		if (UE_TRACE_CHANNELEXPR_IS_ENABLED(TaskChannel))
		{
			if (!bGTaskTraceInitialized)
			{
				return;
			}

			UE_TRACE_LOG(TaskTrace, WaitingStarted, TaskChannel)
				<< WaitingStarted.Timestamp(FPlatformTime::Cycles64())
				<< WaitingStarted.Tasks(&TaskId, 1);
		}
	}

	FWaitingScope::~FWaitingScope()
	{
		if (UE_TRACE_CHANNELEXPR_IS_ENABLED(TaskChannel))
		{
			if (!bGTaskTraceInitialized)
			{
				return;
			}

			UE_TRACE_LOG(TaskTrace, WaitingFinished, TaskChannel)
				<< WaitingFinished.Timestamp(FPlatformTime::Cycles64());
		}
	}

	FTaskTimingEventScope::FTaskTimingEventScope(TaskTrace::FId InTaskId)
	{
		TaskId = InTaskId;
		TaskTrace::Started(TaskId);

		// The RenderingThread outputs a BeginEvent in the BeginFrameRenderThread function and an EndEvent in EndFrameRenderThread.
		// Outputing the ExecuteTask event would cause the Frame event to be closed incorrectly.
#if CPUPROFILERTRACE_ENABLED
		if (UE_TRACE_CHANNELEXPR_IS_ENABLED(TaskChannel|CpuChannel) && !IsInRenderingThread())
		{
			if (ExecuteTaskSpecId == 0)
			{
				ExecuteTaskSpecId = FCpuProfilerTrace::OutputEventType("ExecuteTask", __FILE__, __LINE__);
			}
			FCpuProfilerTrace::OutputBeginEvent(ExecuteTaskSpecId);
			bIsActive = true;
		}
#endif
	}

	FTaskTimingEventScope::~FTaskTimingEventScope()
	{
#if CPUPROFILERTRACE_ENABLED
		if (bIsActive)
		{
			FCpuProfilerTrace::OutputEndEvent();
		}
#endif
		TaskTrace::Finished(TaskId);
	}
}

#endif // UE_TASK_TRACE_ENABLED
