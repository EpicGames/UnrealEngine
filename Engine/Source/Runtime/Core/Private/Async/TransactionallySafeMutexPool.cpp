// Copyright Epic Games, Inc. All Rights Reserved.

#include "Async/TransactionallySafeMutexPool.h"
#include "AutoRTFM.h"
#include "Containers/LockFreeFixedSizeAllocator.h"
#include "HAL/ThreadSafeCounter.h"

#if !UE_BUILD_SHIPPING
using FAllocatorType = TLockFreeFixedSizeAllocator_TLSCache<UE::Private::TransactionallySafeMutexPool::PooledObjectSize, /*TPaddingForCacheContention*/0, FThreadSafeCounter>;
#else
using FAllocatorType = TLockFreeFixedSizeAllocator_TLSCache<UE::Private::TransactionallySafeMutexPool::PooledObjectSize, /*TPaddingForCacheContention*/0>;
#endif

alignas(FAllocatorType) static uint8 GAllocatorStorage[sizeof(FAllocatorType)];

static FAllocatorType& SingletonMutexAllocator()
{
	static FAllocatorType* Allocator = new (GAllocatorStorage) FAllocatorType;
	return *Allocator;
}

void* UE::Private::TransactionallySafeMutexPool::Allocate()
{
	void* Ptr = nullptr;
	UE_AUTORTFM_OPEN
	{
		Ptr = SingletonMutexAllocator().Allocate();
	};
	UE_AUTORTFM_ONABORT(Ptr)
	{
		SingletonMutexAllocator().Free(Ptr);
	};
	return AutoRTFM::DidAllocate(Ptr, PooledObjectSize);
}

void UE::Private::TransactionallySafeMutexPool::Free(void* Ptr)
{
	UE_AUTORTFM_ONCOMMIT(Ptr)
	{
		SingletonMutexAllocator().Free(Ptr);
		AutoRTFM::DidFree(Ptr);
	};
}

#if !UE_BUILD_SHIPPING
int UE::Private::TransactionallySafeMutexPool::GetNumUsed()
{
	return SingletonMutexAllocator().GetNumUsed().GetValue();
}
#endif