// Copyright Epic Games, Inc. All Rights Reserved.

#include "Async/WordMutex.h"

#include "Async/Fundamental/Oversubscription.h"
#include "HAL/PlatformProcess.h"
#include "HAL/PlatformManualResetEvent.h"

namespace UE
{

struct FWordMutexQueueNode
{
	// Points to the next node in the tail-to-head direction. Only null for the current tail.
	FWordMutexQueueNode* Prev = nullptr;
	// Points to the next node in the head-to-tail direction. The tail points to the head.
	// Null until UnlockSlow() has traversed from the tail to fill in next pointers.
	FWordMutexQueueNode* Next = nullptr;

	FPlatformManualResetEvent Event;
};

void FWordMutex::LockSlow()
{
	static_assert((alignof(FWordMutexQueueNode) & QueueMask) == alignof(FWordMutexQueueNode),
		"Alignment of FWordMutexQueueNode is insufficient to pack flags into the lower bits.");

	constexpr int32 SpinLimit = 40;
	int32 SpinCount = 0;
	for (;;)
	{
		UPTRINT CurrentState = State.load(std::memory_order_relaxed);

		// Try to acquire the lock if it was unlocked, even if there is a queue.
		// Acquiring the lock despite the queue means that this lock is not FIFO and thus not fair.
		if (!(CurrentState & IsLockedFlag))
		{
			if (State.compare_exchange_weak(CurrentState, CurrentState | IsLockedFlag, std::memory_order_acquire, std::memory_order_relaxed))
			{
				return;
			}
			continue;
		}

		// Spin up to the spin limit while there is no queue.
		if (!(CurrentState & QueueMask) && SpinCount < SpinLimit)
		{
			FPlatformProcess::Yield();
			++SpinCount;
			continue;
		}

		// Create the node that will be used to add this thread to the queue.
		FWordMutexQueueNode Self;

		Self.Event.Reset();

		// The state points to the tail of the queue, and each node points to the previous node.
		if (FWordMutexQueueNode* Tail = (FWordMutexQueueNode*)(CurrentState & QueueMask))
		{
			Self.Prev = Tail;
		}
		else
		{
			Self.Next = &Self;
		}

		// Swap this thread in as the tail, which makes it visible to any other thread that acquires the queue lock.
		if (!State.compare_exchange_weak(CurrentState, (CurrentState & ~QueueMask) | UPTRINT(&Self), std::memory_order_acq_rel, std::memory_order_relaxed))
		{
			continue;
		}

		// Do not enter oversubscription during a wait on a mutex since the wait is generally too short
		// for it to matter and it can worsen performance a lot for heavily contended locks.
		LowLevelTasks::Private::FOversubscriptionAllowedScope _(false);

		// Wait until another thread wakes this thread, which can happen as soon as the preceding store completes.
		Self.Event.Wait();

		// Loop back and try to acquire the lock.
		SpinCount = 0;
	}
}

void FWordMutex::UnlockSlow(UPTRINT CurrentState)
{
	// IsLockedFlag was cleared by Unlock().
	CurrentState &= ~IsLockedFlag;

	for (;;)
	{
		// Try to lock the queue.
		if (State.compare_exchange_weak(CurrentState, CurrentState | IsQueueLockedFlag, std::memory_order_acquire, std::memory_order_relaxed))
		{
			CurrentState |= IsQueueLockedFlag;
			break;
		}

		// A locked queue indicates that another thread is looking for a thread to wake.
		if ((CurrentState & IsQueueLockedFlag) || !(CurrentState & QueueMask))
		{
			return;
		}
	}

	for (;;)
	{
		// This thread now holds the queue lock. Neither the queue nor State will change while the queue is locked.
		// The state points to the tail of the queue, and each node points to the previous node.
		FWordMutexQueueNode* Tail = (FWordMutexQueueNode*)(CurrentState & QueueMask);

		// Traverse from the tail to find the head and set next pointers for any nodes added since the last unlock.
		for (FWordMutexQueueNode* Node = Tail; !Tail->Next;)
		{
			FWordMutexQueueNode* Prev = Node->Prev;
			checkSlow(Prev);
			Tail->Next = Prev->Next;
			Prev->Next = Node;
			Node = Prev;
		}

		// Another thread may acquire the lock while this thread has been finding a thread to unlock.
		// That case will not be detected on the first iteration of the loop, but only when this
		// thread has failed to unlock the queue at least once. Attempt to unlock the queue here
		// and allow the next unlock to find a thread to wake.
		if (CurrentState & IsLockedFlag)
		{
			if (State.compare_exchange_weak(CurrentState, CurrentState & ~IsQueueLockedFlag, std::memory_order_release, std::memory_order_acquire))
			{
				return;
			}
			continue;
		}

		// The next node from the tail is the head.
		FWordMutexQueueNode* Head = Tail->Next;

		// Remove the head from the queue and unlock the queue.
		if (FWordMutexQueueNode* NewHead = Head->Next; NewHead == Head)
		{
			// Unlock and clear the queue. Failure needs to restart the loop, because newly-added
			// nodes will have a pointer to the node being removed.
			if (!State.compare_exchange_strong(CurrentState, CurrentState & IsLockedFlag, std::memory_order_release, std::memory_order_acquire))
			{
				continue;
			}
		}
		else
		{
			// Clear pointers to the head node being removed.
			checkSlow(NewHead);
			NewHead->Prev = nullptr;
			Tail->Next = NewHead;

			// Unlock the queue regardless of whether new nodes have been added in the meantime.
			State.fetch_and(~IsQueueLockedFlag, std::memory_order_release);
		}

		// Wake the thread that was at the head of the queue.
		Head->Event.Notify();
		break;
	}
}

} // UE
