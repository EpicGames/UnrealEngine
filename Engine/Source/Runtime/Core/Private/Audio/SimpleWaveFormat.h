// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "HAL/Platform.h"

namespace Audio
{
	// Local definition so we don't depend on platform includes.
	enum EFormatType { IEEE_FLOAT = 0x3 }; // WAVE_FORMAT_IEEE_FLOAT
	struct FWaveFormatEx
	{
		uint16	FormatTag;
		uint16	NumChannels;
		uint32	NumSamplesPerSec;
		uint32	AverageBytesPerSec;
		uint16	BlockAlign;
		uint16	NumBitsPerSample;
		uint16	Size;
	};

}