// Copyright Epic Games, Inc. All Rights Reserved.
#include "Audio/SimpleWaveReader.h"
#include "Audio/SimpleWaveFormat.h"
#include "LogAudioCore.h"
#include "HAL/Platform.h"

namespace Audio
{
	FSimpleWaveReader::FSimpleWaveReader(TUniquePtr<FArchive>&& InInputStream)
		: InputStream{ MoveTemp(InInputStream) }
	{
		bIsDataValid = ReadHeader(SampleRate, NumChannels, DataSize);
	}

	FSimpleWaveReader::~FSimpleWaveReader()
	{
	}

	void FSimpleWaveReader::SeekToFrame(uint32 FrameIndex) const
	{
		check(bIsDataValid);
		int64 SeekPos = DataStartPos + FrameIndex * NumChannels * sizeof(float);
		InputStream->Seek(SeekPos);
	}
	
	bool FSimpleWaveReader::Read(TArrayView<const float> OutBuffer, int64& OutNumSamplesRead) const
	{
		check(bIsDataValid);
		
		// buffer is required to be a multiple of the number of channels to read this file
		check(OutBuffer.Num() % NumChannels == 0);

		const int64 BufferDataSize = OutBuffer.NumBytes();
		
		FMemory::Memzero((void*)OutBuffer.GetData(), BufferDataSize);

		if (InputStream->AtEnd())
		{
			OutNumSamplesRead = 0;
			return true;
		}
	
		const int64 DataRemainingSize = DataSize - (InputStream->Tell() - DataStartPos);
		const int64 ReadSize = FMath::Min(BufferDataSize, DataRemainingSize);
		
		InputStream->Serialize((void*)OutBuffer.GetData(), ReadSize);

		OutNumSamplesRead = ReadSize / sizeof(float);

		return InputStream->AtEnd();
	}

	bool FSimpleWaveReader::IsDataValid() const
	{
		return bIsDataValid;
	}

	uint32 FSimpleWaveReader::GetSampleRate() const
	{
		check(bIsDataValid);
		return SampleRate;
	}

	uint16 FSimpleWaveReader::GetNumChannels() const
	{
		check(bIsDataValid);
		return NumChannels;
	}

	uint32 FSimpleWaveReader::GetNumSamples() const
	{
		check(bIsDataValid);
		return DataSize / sizeof(float);
	}

	bool FSimpleWaveReader::ReadHeader(uint32& OutSampleRate, uint16& OutNumChannels, uint32& OutDataSize)
	{
		InputStream->Seek(0);
		
		int32 ID;
		*InputStream << ID;
		if (ID != 'FFIR')
		{
			UE_LOGF(LogAudioCore, Error, "[%s]: Unexpected Format: Expected 'RIFF'", __func__);
			return false;
		}
		
		int32 RiffChunkSize = 0;
		*InputStream << RiffChunkSize;
		
		*InputStream << ID;
		
		if (ID != 'EVAW')
		{
			UE_LOGF(LogAudioCore, Error, "[%s]: Unexpected Format - Expected 'WAVE', was '%s'", __func__, (ANSICHAR*)&ID);
			return false;
		}
		
		*InputStream << ID;
		if (ID != ' tmf')
		{
			UE_LOGF(LogAudioCore, Error, "[%s]: Unexpected chunk - Expected 'fmt ', was '%s'", __func__, (ANSICHAR*)&ID);
			return false;
		}
		
		FWaveFormatEx Fmt = { 0 };
		int32 FmtSize = sizeof(Fmt);
		*InputStream << FmtSize;
		InputStream->Serialize((void*)&Fmt, FmtSize);

		uint16 NumBitsPerSample = sizeof(float) * 8;
		if (Fmt.NumBitsPerSample != NumBitsPerSample)
		{
			UE_LOGF(LogAudioCore, Error, "[%s]: Expected NumBitsPerSample to be %d was %d", __func__, NumBitsPerSample, Fmt.NumBitsPerSample);
			return false;
		}

		uint16 BlockAlign = (uint16)(Fmt.NumBitsPerSample * Fmt.NumChannels) / 8;
		if (Fmt.BlockAlign != BlockAlign)
		{
			UE_LOGF(LogAudioCore, Error, "[%s]: Expected BlockAlign to be %d was %d", __func__, BlockAlign, Fmt.BlockAlign);
			return false;
		}

		uint32 AverageBytesPerSec = Fmt.BlockAlign * Fmt.NumSamplesPerSec;
		if (Fmt.AverageBytesPerSec != AverageBytesPerSec)
		{
			UE_LOGF(LogAudioCore, Error, "[%s]: Expected AverageBytesPerSec to be %d was %d", __func__, AverageBytesPerSec, Fmt.AverageBytesPerSec);
			return false;
		}
		
		if (Fmt.FormatTag != EFormatType::IEEE_FLOAT)
		{
			UE_LOGF(LogAudioCore, Error, "[%s]: Expected FormatTag to be %d was %d", __func__, EFormatType::IEEE_FLOAT, Fmt.FormatTag);
			return false;
		}
		
		*InputStream << ID;
		if (ID != 'atad')
		{
			UE_LOGF(LogAudioCore, Error, "[%s]: Unexpected chunk - expected 'data', was '%s'", __func__, (ANSICHAR*)&ID);
			return false;
		}
		
		*InputStream << OutDataSize;

		if (OutDataSize == 0)
		{
			UE_LOGF(LogAudioCore, Error, "[%s]: File contains no wav data", __func__);
			return false;
		}
		
		if (Fmt.NumSamplesPerSec == 0)
		{
			UE_LOGF(LogAudioCore, Error, "[%s]: Invalid Sample Rate %d", __func__, Fmt.NumSamplesPerSec);
			return false;
		}
		
		if (Fmt.NumChannels == 0)
		{
			UE_LOGF(LogAudioCore, Error, "[%s]: Invalid Num Channels %d", __func__, Fmt.NumChannels);
			return false;
		}

		OutSampleRate = Fmt.NumSamplesPerSec;
		OutNumChannels = Fmt.NumChannels;
		
		DataStartPos = InputStream->Tell();
		return true;
	}
}