// Copyright Epic Games, Inc. All Rights Reserved.
#include "Audio/SimpleWaveWriter.h"
#include "Audio/SimpleWaveFormat.h"

namespace Audio
{
	FSimpleWaveWriter::FSimpleWaveWriter(TUniquePtr<FArchive>&& InOutputStream, int32 InSampleRate, int32 InNumChannels, bool bInUpdateHeaderAfterEveryWrite)
		: OutputStream{ MoveTemp(InOutputStream) }
		, bUpdateHeaderAfterEveryWrite{ bInUpdateHeaderAfterEveryWrite }
	{
		WriteHeader(InSampleRate, InNumChannels);
	}

	FSimpleWaveWriter::~FSimpleWaveWriter()
	{
		UpdateHeader();
	}

	void FSimpleWaveWriter::Write(TArrayView<const float> InBuffer)
	{
		OutputStream->Serialize((void*)InBuffer.GetData(), InBuffer.GetTypeSize() * InBuffer.Num());

		if (bUpdateHeaderAfterEveryWrite)
		{
			UpdateHeader();
		}
	}

	void FSimpleWaveWriter::UpdateHeader()
	{
		// RIFF/fmt/data. (bytes per chunk)
		static const int32 HeaderSize = sizeof(FWaveFormatEx) + sizeof(int32) + sizeof(int32) + sizeof(int32) + sizeof(int32) + sizeof(int32);

		int32 WritePos = (uint32)OutputStream->Tell();

		// update data chunk size
		OutputStream->Seek(DataSizePos);
		int32 DataSize = WritePos - DataSizePos - 4;
		*OutputStream << DataSize;

		// update top riff size
		OutputStream->Seek(RiffSizePos);
		int32 RiffSize = HeaderSize + DataSize - 4;
		*OutputStream << RiffSize;

		OutputStream->Seek(WritePos);
	}

	void FSimpleWaveWriter::WriteHeader(int32 InSampleRate, int32 InNumChannels)
	{
		FWaveFormatEx Fmt = { 0 };
		Fmt.NumChannels = (uint16)InNumChannels;
		Fmt.NumSamplesPerSec = InSampleRate;
		Fmt.NumBitsPerSample = sizeof(float) * 8;
		Fmt.BlockAlign = (uint16)((Fmt.NumBitsPerSample * InNumChannels) / 8);
		Fmt.AverageBytesPerSec = Fmt.BlockAlign * InSampleRate;
		Fmt.FormatTag = EFormatType::IEEE_FLOAT;// WAVE_FORMAT_IEEE_FLOAT;

		int32 ID = 'FFIR';
		*OutputStream << ID;
		RiffSizePos = (int32)OutputStream->Tell();
		int32 RiffChunkSize = 0;
		*OutputStream << RiffChunkSize;

		ID = 'EVAW';
		*OutputStream << ID;

		ID = ' tmf';
		*OutputStream << ID;
		int32 FmtSize = sizeof(Fmt);
		*OutputStream << FmtSize;
		OutputStream->Serialize((void*)&Fmt, FmtSize);

		ID = 'atad';
		*OutputStream << ID;
		DataSizePos = (int32)OutputStream->Tell();
		int32 DataChunkSize = 0;
		*OutputStream << DataChunkSize;
	}
}