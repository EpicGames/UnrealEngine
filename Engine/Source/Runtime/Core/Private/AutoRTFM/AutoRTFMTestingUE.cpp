// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutoRTFM/AutoRTFMTestingUE.h"

#include "CoreGlobals.h"
#include "HAL/PlatformStackWalk.h"

#if AUTORTFM_ENABLE_TEST_UTILS

namespace AutoRTFM
{
#if UE_AUTORTFM
	// Declared in AutoRTFMUE.cpp
	extern TFunction<void(autortfm_extern_api&)> GInterceptExternAPIsForTesting;
#endif
}

namespace AutoRTFM::Testing
{

void InterceptExternAPIs([[maybe_unused]] TFunction<void(autortfm_extern_api&)> Intercept)
{
#if UE_AUTORTFM
	GInterceptExternAPIsForTesting = Intercept;
#endif
}

void PrintCallstackOnCrash()
{
	// Enable guarded execution so that check/ensure failures raise SEH exceptions
	// (caught by the VEH handler installed below) instead of logging and terminating the process.
	GIsGuarded = true;

	OnCrash([](const FCrashInfo& Info)
	{
		static constexpr uint32 MaxCallstackDepth = 256;
		uint64 Callstack[MaxCallstackDepth] = {};
		uint32 NumFrames = FPlatformStackWalk::CaptureStackBackTrace(Callstack, MaxCallstackDepth);
		uint64* Frames = Callstack;
		for (uint32 FrameIndex = 0; FrameIndex < NumFrames; FrameIndex++)
		{
			if (Frames[FrameIndex] == reinterpret_cast<uint64>(Info.ProgramCounter))
			{
				Frames += FrameIndex;
				NumFrames -= FrameIndex;
				break;
			}
		}
		fprintf(stderr, "=== Critical error: %s ===\n", Info.Kind);
		for (uint32 FrameIndex = 0; FrameIndex < NumFrames; FrameIndex++)
		{
			static constexpr uint32 StringSize = 1024;
			char String[StringSize] = {};
			FPlatformStackWalk::ProgramCounterToHumanReadableString(
				static_cast<int32>(FrameIndex), Frames[FrameIndex], String, StringSize);
			fprintf(stderr, "%s\n", String);
		}
	});
}

}  // AutoRTFM::Testing

#endif // AUTORTFM_ENABLE_TEST_UTILS
