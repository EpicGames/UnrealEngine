// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutoRTFM/AutoRTFMUE.h"
#include "HAL/Platform.h"
#include <cstdint>

#if UE_AUTORTFM

#include "AutoRTFM.h"
#include "Containers/StringConv.h"
#include "Containers/UnrealString.h"
#include "GenericPlatform/GenericPlatformCrashContext.h"
#include "GenericPlatform/GenericPlatformStackWalk.h"
#include "HAL/IConsoleManager.h"
#include "HAL/LowLevelMemTracker.h"
#include "HAL/UnrealMemory.h"
#include "Logging/LogVerbosity.h"
#include "Misc/AssertionMacros.h"
#include "Templates/Function.h"

#include <cstdarg>
#include <cstdio>
#include <memory>

DEFINE_LOG_CATEGORY(LogAutoRTFM)

static_assert(UE_AUTORTFM_ENABLED, "AutoRTFM/AutoRTFMUE.cpp requires the compiler flag '-fautortfm'");

namespace AutoRTFM
{

namespace
{

static bool IsDisable(const FString& String)
{
	return String == "disable" || String == "disabled" || String == "false" || String == "off" || String == "0";
}

static bool IsEnable(const FString& String)
{
	return String == "enable" || String == "enabled" || String == "true" || String == "on" || String == "1";
}

void OnAutoRTFMRuntimeEnabledChanged()
{
	const bool bEnabled = AutoRTFM::ForTheRuntime::IsAutoRTFMRuntimeEnabled();
	UE_LOGF(LogAutoRTFM, Log, "AutoRTFM: %ls", bEnabled ? TEXT("enabled") : TEXT("disabled"));
	FGenericCrashContext::SetGameData(TEXT("IsAutoRTFMRuntimeEnabled"), bEnabled ? TEXT("true") : TEXT("false"));
}

void OnAutoRTFMRetryTransactionsChanged()
{
	AutoRTFM::ForTheRuntime::EAutoRTFMRetryTransactionState Value = AutoRTFM::ForTheRuntime::GetRetryTransaction();
	switch (Value)
	{
	case AutoRTFM::ForTheRuntime::EAutoRTFMRetryTransactionState::NoRetry:
		UE_LOGF(LogAutoRTFM, Log, "AutoRTFM Retry Transactions: disabled");
		return FGenericCrashContext::SetGameData(TEXT("AutoRTFMRetryTransactionState"), TEXT("NoRetry"));
	case AutoRTFM::ForTheRuntime::EAutoRTFMRetryTransactionState::RetryNonNested:
		UE_LOGF(LogAutoRTFM, Log, "AutoRTFM Retry Transactions: retry-non-nested");
		return FGenericCrashContext::SetGameData(TEXT("AutoRTFMRetryTransactionState"), TEXT("RetryNonNested"));
	case AutoRTFM::ForTheRuntime::EAutoRTFMRetryTransactionState::RetryNestedToo:
		UE_LOGF(LogAutoRTFM, Log, "AutoRTFM Retry Transactions: retry-nested-too");
		return FGenericCrashContext::SetGameData(TEXT("AutoRTFMRetryTransactionState"), TEXT("RetryNestedToo"));
	}
}

#if AUTORTFM_SANITIZER
void OnAutoRTFMSanitizerModeChanged()
{
	switch (AutoRTFM::Sanitizer::GetMode())
	{
	case AutoRTFM::Sanitizer::EMode::Disabled:
		UE_LOGF(LogAutoRTFM, Log, "AutoRTFM Sanitizer: disabled");
		return FGenericCrashContext::SetGameData(TEXT("AutoRTFMSanitizerMode"), TEXT("Disabled"));
	case AutoRTFM::Sanitizer::EMode::Warn:
		UE_LOGF(LogAutoRTFM, Log, "AutoRTFM Sanitizer: enabled as warning");
		return FGenericCrashContext::SetGameData(TEXT("AutoRTFMSanitizerMode"), TEXT("Warn"));
	case AutoRTFM::Sanitizer::EMode::Error:
		UE_LOGF(LogAutoRTFM, Log, "AutoRTFM Sanitizer: enabled as error");
		return FGenericCrashContext::SetGameData(TEXT("AutoRTFMSanitizerMode"), TEXT("Enabled"));
	}
}

void OnAutoRTFMSanitizerRecordClosedWriteCallstacksChanged()
{
	UE_LOGF(LogAutoRTFM, Log, "AutoRTFM Sanitizer: %ls recording of closed write callstacks",
		AutoRTFM::Sanitizer::RecordClosedWriteCallstacks() ? TEXT("enabled") : TEXT("disabled"));
}

#endif // AUTORTFM_SANITIZER

static FAutoConsoleVariable CVarAutoRTFMRuntimeEnabled(
	TEXT("AutoRTFMRuntimeEnabled"),
	TEXT("default"),
	TEXT("Enables the AutoRTFM runtime"),
	FConsoleVariableDelegate::CreateLambda([] (IConsoleVariable* Variable)
	{
		FString Value = Variable->GetString().ToLower();
		if (Value == "default")
		{
			return;
		}
		if (Value == "forceon")
		{
			AutoRTFM::ForTheRuntime::SetAutoRTFMRuntime(AutoRTFM::ForTheRuntime::AutoRTFM_ForcedEnabled);
			return;
		}
		if (Value == "forceoff")
		{
			AutoRTFM::ForTheRuntime::SetAutoRTFMRuntime(AutoRTFM::ForTheRuntime::AutoRTFM_ForcedDisabled);
			return;
		}
		if (Value == "1") // The CVar system converts On to '1'.
		{
			AutoRTFM::ForTheRuntime::SetAutoRTFMRuntime(AutoRTFM::ForTheRuntime::AutoRTFM_Enabled);
			return;
		}
		if (Value == "0") // The CVar system converts Off to '0'.
		{
			AutoRTFM::ForTheRuntime::SetAutoRTFMRuntime(AutoRTFM::ForTheRuntime::AutoRTFM_Disabled);
			return;
		}
		if (Value == "2")
		{
			AutoRTFM::ForTheRuntime::SetAutoRTFMRuntime(AutoRTFM::ForTheRuntime::AutoRTFM_ForcedDisabled);
			return;
		}
		if (Value == "3")
		{
			AutoRTFM::ForTheRuntime::SetAutoRTFMRuntime(AutoRTFM::ForTheRuntime::AutoRTFM_ForcedEnabled);
			return;
		}
		UE_LOGF(LogAutoRTFM, Fatal, "'AutoRTFMRuntimeEnabled' CVar was set to '%ls' which is not one of 'ForceOn', 'ForceOff', 'On', or 'Off'!", *Value);
	}),
	ECVF_Default
);

static FAutoConsoleVariable CVarAutoRTFMInternalAbortAction(
	TEXT("AutoRTFMInternalAbortAction"),
	TEXT("default"),
	TEXT("If true when we hit an AutoRTFM issue assert over ensuring"),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* Variable)
	{
		FString Value = Variable->GetString().ToLower();
		if (Value == "default")
		{
			return;
		}
		if (Value == "crash")
		{
			AutoRTFM::ForTheRuntime::SetInternalAbortAction(AutoRTFM::ForTheRuntime::EAutoRTFMInternalAbortActionState::Crash);
			return;
		}
		if (Value == "abort")
		{
			AutoRTFM::ForTheRuntime::SetInternalAbortAction(AutoRTFM::ForTheRuntime::EAutoRTFMInternalAbortActionState::Abort);
			return;
		}
		UE_LOGF(LogAutoRTFM, Fatal, "'AutoRTFMInternalAbortAction' CVar was set to '%ls' which is not one of 'Crash' or 'Abort'!", *Value);

	})
);

static FAutoConsoleVariable CVarAutoRTFMRetryTransactions(
	TEXT("AutoRTFMRetryTransactions"),
	static_cast<int>(AutoRTFM::ForTheRuntime::GetRetryTransaction()),
	TEXT("Enables the AutoRTFM sanitizer-like mode where we can force an abort-and-retry on transactions (useful to test abort codepaths work as intended)"),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* Variable)
	{
		const AutoRTFM::ForTheRuntime::EAutoRTFMRetryTransactionState Value = static_cast<AutoRTFM::ForTheRuntime::EAutoRTFMRetryTransactionState>(Variable->GetInt());
		AutoRTFM::ForTheRuntime::SetRetryTransaction(Value);
	}),
	ECVF_Default
);

#if AUTORTFM_SANITIZER
static FAutoConsoleVariable CVarAutoRTFMSanitizerMode(
	TEXT("AutoRTFMSanitizerMode"),
	static_cast<int>(AutoRTFM::Sanitizer::GetMode()),
	TEXT("Detects potential memory corruption due to writes made both by a transaction and open-code"),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* Variable)
	{
		FString Value = Variable->GetString().ToLower();
		if (IsDisable(Value))
		{
			AutoRTFM::Sanitizer::SetMode(AutoRTFM::Sanitizer::EMode::Disabled);
			return;
		}
		if (Value == "warn")
		{
			AutoRTFM::Sanitizer::SetMode(AutoRTFM::Sanitizer::EMode::Warn);
			return;
		}
		if (Value == "error" || IsEnable(Value))
		{
			AutoRTFM::Sanitizer::SetMode(AutoRTFM::Sanitizer::EMode::Error);
			return;
		}
		UE_LOGF(LogAutoRTFM, Fatal, "'AutoRTFMSanitizerMode' CVar was set to '%ls' which is not one of 'disable', 'warn' or 'error'!", *Value);
	}),
	ECVF_Default
);

static FAutoConsoleVariable CVarAutoRTFMSanitizerRecordWriteCallstacks(
	TEXT("AutoRTFMSanitizerRecordWriteCallstacks"),
	AutoRTFM::Sanitizer::RecordClosedWriteCallstacks(),
	TEXT("Enables the recording of callstacks for closed writes, providing more information on AutoRTFMSan failures"),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* Variable)
	{
		FString Value = Variable->GetString().ToLower();
		if (IsDisable(Value))
		{
			AutoRTFM::Sanitizer::SetRecordClosedWriteCallstacks(false);
			return;
		}
		if (IsEnable(Value))
		{
			AutoRTFM::Sanitizer::SetRecordClosedWriteCallstacks(true);
			return;
		}
		UE_LOGF(LogAutoRTFM, Fatal, "'AutoRTFMSanitizerRecordWriteCallstacks' CVar was set to '%ls' which is not one of 'enable' or 'disable'!", *Value);
	}),
	ECVF_Default
);
#endif

static FAutoConsoleVariable CVarAutoRTFMEnabledProbability(
	TEXT("AutoRTFMEnabledProbability"),
	AutoRTFM::ForTheRuntime::GetAutoRTFMEnabledProbability(),
	TEXT("A rational percentage from [0..100] of what threshold to `CoinTossDisable` AutoRTFM. 100 means always enable, 0 means always disable"),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* Variable)
	{
		const float Value = Variable->GetFloat();
		AutoRTFM::ForTheRuntime::SetAutoRTFMEnabledProbability(Value);
	}),
	ECVF_Default
);

bool AutoRTFMIsSeverityActive(autortfm_log_severity Severity)
{
	switch (Severity)
	{
		case autortfm_log_verbose:
			return UE_LOG_ACTIVE(LogAutoRTFM, Verbose);
		case autortfm_log_info:
			return UE_LOG_ACTIVE(LogAutoRTFM, Display);
		case autortfm_log_warn:
			return UE_LOG_ACTIVE(LogAutoRTFM, Warning);
		case autortfm_log_error:
			return UE_LOG_ACTIVE(LogAutoRTFM, Error);
		case autortfm_log_fatal:
			return UE_LOG_ACTIVE(LogAutoRTFM, Fatal);
	}
}

UE_AUTORTFM_ALWAYS_OPEN_NO_MEMORY_VALIDATION
TStringConversion<TStringConvert<UTF8CHAR, TCHAR>, 128> FmtToTChar(const char* Format, va_list Args)
{
	static constexpr size_t InlineBufferLength = 256;
	char InlineBuffer[InlineBufferLength];

	va_list Args2;
	va_copy(Args2, Args);
	int Count = vsnprintf(InlineBuffer, InlineBufferLength, Format, Args);

	if (Count < InlineBufferLength)
	{
		va_end(Args2);
		return StringCast<TCHAR>(reinterpret_cast<const UTF8CHAR*>(InlineBuffer));
	}
	else
	{
		std::unique_ptr<char[]> Buffer{new char[Count+1]};
		vsnprintf(Buffer.get(), Count+1, Format, Args2);
		va_end(Args2);
		return StringCast<TCHAR>(reinterpret_cast<const UTF8CHAR*>(Buffer.get()));
	}
}

UE_AUTORTFM_ALWAYS_OPEN_NO_MEMORY_VALIDATION
void AutoRTFMLogV(const char* File, int Line, void* ProgramCounter, autortfm_log_severity Severity, const char* Format, va_list Args)
{
#if !NO_LOGGING
	if (!AutoRTFMIsSeverityActive(Severity))
	{
		return;
	}

	static ::UE::Logging::Private::FStaticBasicLogDynamicData DynamicData;
	UE::Logging::Private::TStaticBasicLogRecord<TCHAR> Record(
		/* Format */ TEXT("%s"),
		/* File */ File,
		/* Line */ Line,
		/* DynamicData */ &DynamicData
	);

	auto Message = FmtToTChar(Format, Args);

	switch (Severity)
	{
		case autortfm_log_verbose:
			UE::Logging::Private::BasicLog<ELogVerbosity::Verbose>(Record, &LogAutoRTFM, Message.Get());
			break;
		case autortfm_log_info:
			UE::Logging::Private::BasicLog<ELogVerbosity::Display>(Record, &LogAutoRTFM, Message.Get());
			break;
		case autortfm_log_warn:
			UE::Logging::Private::BasicLog<ELogVerbosity::Warning>(Record, &LogAutoRTFM, Message.Get());
			break;
		case autortfm_log_error:
			UE::Logging::Private::BasicLog<ELogVerbosity::Error>(Record, &LogAutoRTFM, Message.Get());
			break;
		case autortfm_log_fatal:
			UE::Logging::Private::BasicFatalLogWithProgramCounter(Record, &LogAutoRTFM, ProgramCounter, Message.Get());
			break;
	}
#endif
}

void AutoRTFMLogf(const char* File, int Line, void* ProgramCounter, autortfm_log_severity Severity, const char* Format, ...)
{
	va_list Args;
	va_start(Args, Format);
	AutoRTFMLogV(File, Line, ProgramCounter, Severity, Format, Args);
	va_end(Args);
}

size_t AutoRTFMCaptureCallstack(size_t MaxCount, autortfm_callstack_frame* StackOut)
{
	return FPlatformStackWalk::CaptureStackBackTrace(StackOut, static_cast<uint32>(MaxCount));
}

void AutoRTFMLogCallstack(autortfm_log_severity Severity, size_t Count, autortfm_callstack_frame const* Stack)
{
#if !NO_LOGGING
	for (size_t CurrentDepth = 0; CurrentDepth < Count; CurrentDepth++)
	{
		static constexpr size_t StringSize = 1024;
		ANSICHAR String[StringSize] {'\0'};
		FPlatformStackWalk::ProgramCounterToHumanReadableString(CurrentDepth, Stack[CurrentDepth], String, StringSize);
		AutoRTFMLogf(/* File */ nullptr, /* Line */ 0, /* ProgramCounter */ nullptr, Severity, "%s", String);
	}
#endif
}

UE_AUTORTFM_ALWAYS_OPEN_NO_MEMORY_VALIDATION
void AutoRTFMLogWithCallstack(void* ProgramCounter, autortfm_log_severity Severity, const char* Format, va_list Args)
{
#if !NO_LOGGING
	if (!AutoRTFMIsSeverityActive(Severity))
	{
		return;
	}

	AutoRTFMLogV(/* File */ nullptr, /* Line */ 0, ProgramCounter, Severity, Format, Args);
	constexpr uint32 MaxBacktrace = 512;
	uint64 Backtrace[MaxBacktrace];
	uint32 Count = FPlatformStackWalk::CaptureStackBackTrace(Backtrace, MaxBacktrace);
	uint32 NumFramesToSkip = 2;
	if (ProgramCounter)
	{
		for (uint32 I = 0; I < Count; I++)
		{
			if (Backtrace[I] == static_cast<uint64>(reinterpret_cast<uintptr_t>(ProgramCounter)))
			{
				NumFramesToSkip = I;
				break;
			}
		}
	}

	if (Count > NumFramesToSkip)
	{
		AutoRTFMLogCallstack(Severity, Count - NumFramesToSkip, Backtrace + NumFramesToSkip);
	}
#endif
}

UE_AUTORTFM_ALWAYS_OPEN_NO_MEMORY_VALIDATION
void AutoRTFMEnsureFailure(const char* File, int Line, void* ProgramCounter, const char* Condition, const char* Format, va_list Args)
{
#if DO_ENSURE
	if (Format == nullptr)
	{
		Format = "";
	}
	FDebug::DumpStackTraceToLog(TEXT("AutoRTFM backtrace"), ELogVerbosity::Error);
	FDebug::EnsureFailed(Condition, File, Line, ProgramCounter, FmtToTChar(Format, Args).Get());
#endif
}

} // anonymous namespace

LLM_DEFINE_TAG(AutoRTFM);

// Set by InterceptExternAPIs() in AutoRTFMTestingUE.cpp
TFunction<void(autortfm_extern_api&)> GInterceptExternAPIsForTesting;

static void* AutoRTFMAllocate(size_t Size, size_t Alignment)
{
	LLM_SCOPE_BYTAG(AutoRTFM);
	return FMemory::Malloc(Size, Alignment);
}

static void* AutoRTFMReallocate(void* Pointer, size_t Size, size_t Alignment, size_t PreviousSize)
{
	(void) PreviousSize;
	LLM_SCOPE_BYTAG(AutoRTFM);
	return FMemory::Realloc(Pointer, Size, Alignment);
}

static void* AutoRTFMAllocateZeroed(size_t Size, size_t Alignment)
{
	LLM_SCOPE_BYTAG(AutoRTFM);
	return FMemory::MallocZeroed(Size, Alignment);
}

static void AutoRTFMFree(void* Pointer, size_t Size)
{
	(void) Size;
	LLM_SCOPE_BYTAG(AutoRTFM);
	FMemory::Free(Pointer);
}

CORE_API void InitializeForUE(RollbackWriteFn RollbackWrite /* = nullptr */)
{
	AutoRTFM::FExternAPI ExternAPI
	{
		.Allocate = AutoRTFMAllocate,
		.Reallocate = AutoRTFMReallocate,
		.AllocateZeroed = AutoRTFMAllocateZeroed,
		.Free = AutoRTFMFree,
		.Log = AutoRTFMLogV,
		.LogWithCallstack = AutoRTFMLogWithCallstack,
		.EnsureFailure = AutoRTFMEnsureFailure,
		.IsLogActive = AutoRTFMIsSeverityActive,
		.CaptureCallstack = AutoRTFMCaptureCallstack,
		.LogCallstack = AutoRTFMLogCallstack,
		.OnRuntimeEnabledChanged = OnAutoRTFMRuntimeEnabledChanged,
		.OnRetryTransactionsChanged = OnAutoRTFMRetryTransactionsChanged,
#if AUTORTFM_SANITIZER
		.OnSanitizerModeChanged = OnAutoRTFMSanitizerModeChanged,
		.OnSanitizerRecordClosedWriteCallstacksChanged = OnAutoRTFMSanitizerRecordClosedWriteCallstacksChanged,
#endif
		.RollbackWrite = RollbackWrite,
	};

	if (GInterceptExternAPIsForTesting.IsSet())
	{
		GInterceptExternAPIsForTesting(ExternAPI);
	}
	
	AutoRTFM::Initialize(ExternAPI);

	// Call the OnAutoRTFMXXXChanged() handlers now so that values are logged
	// and crash context data is set.
	OnAutoRTFMRuntimeEnabledChanged();
	OnAutoRTFMRetryTransactionsChanged();
#if AUTORTFM_SANITIZER
	OnAutoRTFMSanitizerModeChanged();
	OnAutoRTFMSanitizerRecordClosedWriteCallstacksChanged();
#endif
}

} // namespace AutoRTFM

#endif // UE_AUTORTFM
