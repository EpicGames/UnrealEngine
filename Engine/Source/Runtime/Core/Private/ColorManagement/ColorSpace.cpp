// Copyright Epic Games, Inc. All Rights Reserved.

#include "ColorManagement/ColorSpace.h"

#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Misc/ConfigCacheIni.h"
#include "Math/VectorRegister.h"

namespace UE { namespace Color {

static bool bIsWorkingColorSpaceReadyForUse = false;
static FColorSpace WorkingColorSpace = FColorSpace(EColorSpace::sRGB);

void PreloadWorkingColorSpace()
{
	if (bIsWorkingColorSpaceReadyForUse)
	{
		return;
	}

	// Note: Other non-engine programs may not have access project configs. For those edge cases, we must fall back to default.
	if (GConfig != nullptr)
	{
		check(GConfig->IsReadyForUse());

		int32 LoadedChromaticities = 0;
		TStaticArray<FVector2d, 4> Chromaticities;
		LoadedChromaticities += static_cast<int32>(GConfig->GetVector2D(TEXT("/Script/Engine.RendererSettings"), TEXT("RedChromaticityCoordinate"), Chromaticities[0], GEngineIni));
		LoadedChromaticities += static_cast<int32>(GConfig->GetVector2D(TEXT("/Script/Engine.RendererSettings"), TEXT("GreenChromaticityCoordinate"), Chromaticities[1], GEngineIni));
		LoadedChromaticities += static_cast<int32>(GConfig->GetVector2D(TEXT("/Script/Engine.RendererSettings"), TEXT("BlueChromaticityCoordinate"), Chromaticities[2], GEngineIni));
		LoadedChromaticities += static_cast<int32>(GConfig->GetVector2D(TEXT("/Script/Engine.RendererSettings"), TEXT("WhiteChromaticityCoordinate"), Chromaticities[3], GEngineIni));

		if (LoadedChromaticities == 0)
		{
			// No-op: The working color space wasn't modified/serialized to the config so we keep the implicit sRGB default.
		}
		else if (LoadedChromaticities == 4)
		{
			FColorSpace::SetWorking(FColorSpace(Chromaticities[0], Chromaticities[1], Chromaticities[2], Chromaticities[3]));
		}
		else
		{
			UE_LOGF(LogCore, Warning, "Working color space in config is missing chromaticities, falling back to sRGB.");
		}
	}
	else
	{
		UE_LOGF(LogCore, Display, "Working color space in config is not available, falling back to sRGB.");
	}

	bIsWorkingColorSpaceReadyForUse = true;
};

const FColorSpace& FColorSpace::GetWorking()
{
	/**
	 * NOTE: Addresses issue where shader compilation can request the working color space before it has been loaded by renderer settings.
	 * We optionally early-load the settings here and leave existing singleton interface as is.
	 */
	UE_CALL_ONCE(PreloadWorkingColorSpace);

	return WorkingColorSpace;
}

void FColorSpace::SetWorking(FColorSpace ColorSpace)
{
	WorkingColorSpace = MoveTemp(ColorSpace);
	bIsWorkingColorSpaceReadyForUse = true;
}

const FColorSpace& FColorSpace::GetSRGB()
{
	static const FColorSpace ColorSpaceSRGB = FColorSpace(EColorSpace::sRGB);

	return ColorSpaceSRGB;
}

const FColorSpace& FColorSpace::GetRec2020()
{
	static const FColorSpace ColorSpaceRec2020 = FColorSpace(EColorSpace::Rec2020);

	return ColorSpaceRec2020;
}

const FColorSpace& FColorSpace::GetP3D65()
{
	static const FColorSpace ColorSpaceP3D65 = FColorSpace(EColorSpace::P3D65);

	return ColorSpaceP3D65;
}

static bool IsSRGBChromaticities(const TStaticArray<FVector2d, 4>& Chromaticities, double Tolerance = 1.e-7)
{
	return	Chromaticities[0].Equals(FVector2d(0.64, 0.33), Tolerance) &&
		Chromaticities[1].Equals(FVector2d(0.30, 0.60), Tolerance) &&
		Chromaticities[2].Equals(FVector2d(0.15, 0.06), Tolerance) &&
		Chromaticities[3].Equals(FVector2d(0.3127, 0.3290), Tolerance);
}

FColorSpace::FColorSpace(const FVector2d& InRed, const FVector2d& InGreen, const FVector2d& InBlue, const FVector2d& InWhite)
{
	Chromaticities[0] = InRed;
	Chromaticities[1] = InGreen;
	Chromaticities[2] = InBlue;
	Chromaticities[3] = InWhite;

	bIsSRGB = IsSRGBChromaticities(Chromaticities);

	RgbToXYZ = CalcRgbToXYZ();
	XYZToRgb = RgbToXYZ.Inverse();
}

FColorSpace::FColorSpace(EColorSpace ColorSpaceType)
	: Chromaticities(MakeChromaticities(ColorSpaceType))
	, bIsSRGB(ColorSpaceType == EColorSpace::sRGB)
{
	RgbToXYZ = CalcRgbToXYZ();
	XYZToRgb = RgbToXYZ.Inverse();
}

TStaticArray<FVector2d, 4> FColorSpace::MakeChromaticities(UE::Color::EColorSpace ColorSpaceType)
{
	TStaticArray<FVector2d, 4> Chromaticities(InPlace, FVector2d::Zero());

	switch (ColorSpaceType)
	{
	case EColorSpace::None:
		// leaves FVector2d::Zero from the constructor
		break;
	case EColorSpace::sRGB:
		Chromaticities[0] = FVector2d(0.64, 0.33);
		Chromaticities[1] = FVector2d(0.30, 0.60);
		Chromaticities[2] = FVector2d(0.15, 0.06);
		Chromaticities[3] = GetWhitePoint(EWhitePoint::CIE1931_D65);
		break;
	case EColorSpace::Rec2020:
		Chromaticities[0] = FVector2d(0.708, 0.292);
		Chromaticities[1] = FVector2d(0.170, 0.797);
		Chromaticities[2] = FVector2d(0.131, 0.046);
		Chromaticities[3] = GetWhitePoint(EWhitePoint::CIE1931_D65);
		break;
	case EColorSpace::ACESAP0:
		Chromaticities[0] = FVector2d(0.7347, 0.2653);
		Chromaticities[1] = FVector2d(0.0000, 1.0000);
		Chromaticities[2] = FVector2d(0.0001, -0.0770);
		Chromaticities[3] = GetWhitePoint(EWhitePoint::ACES_D60);
		break;
	case EColorSpace::ACESAP1:
		Chromaticities[0] = FVector2d(0.713, 0.293);
		Chromaticities[1] = FVector2d(0.165, 0.830);
		Chromaticities[2] = FVector2d(0.128, 0.044);
		Chromaticities[3] = GetWhitePoint(EWhitePoint::ACES_D60);
		break;
	case EColorSpace::P3DCI:
		Chromaticities[0] = FVector2d(0.680, 0.320);
		Chromaticities[1] = FVector2d(0.265, 0.690);
		Chromaticities[2] = FVector2d(0.150, 0.060);
		Chromaticities[3] = GetWhitePoint(EWhitePoint::DCI_CalibrationWhite);
		break;
	case EColorSpace::P3D65:
		Chromaticities[0] = FVector2d(0.680, 0.320);
		Chromaticities[1] = FVector2d(0.265, 0.690);
		Chromaticities[2] = FVector2d(0.150, 0.060);
		Chromaticities[3] = GetWhitePoint(EWhitePoint::CIE1931_D65);
		break;
	case EColorSpace::REDWideGamut:
		Chromaticities[0] = FVector2d(0.780308, 0.304253);
		Chromaticities[1] = FVector2d(0.121595, 1.493994);
		Chromaticities[2] = FVector2d(0.095612, -0.084589);
		Chromaticities[3] = GetWhitePoint(EWhitePoint::CIE1931_D65);
		break;
	case EColorSpace::SonySGamut3:
		Chromaticities[0] = FVector2d(0.730, 0.280);
		Chromaticities[1] = FVector2d(0.140, 0.855);
		Chromaticities[2] = FVector2d(0.100, -0.050);
		Chromaticities[3] = GetWhitePoint(EWhitePoint::CIE1931_D65);
		break;
	case EColorSpace::SonySGamut3Cine:
		Chromaticities[0] = FVector2d(0.766, 0.275);
		Chromaticities[1] = FVector2d(0.225, 0.800);
		Chromaticities[2] = FVector2d(0.089, -0.087);
		Chromaticities[3] = GetWhitePoint(EWhitePoint::CIE1931_D65);
		break;
	case EColorSpace::AlexaWideGamut:
		Chromaticities[0] = FVector2d(0.684, 0.313);
		Chromaticities[1] = FVector2d(0.221, 0.848);
		Chromaticities[2] = FVector2d(0.0861, -0.1020);
		Chromaticities[3] = GetWhitePoint(EWhitePoint::CIE1931_D65);
		break;
	case EColorSpace::CanonCinemaGamut:
		Chromaticities[0] = FVector2d(0.740, 0.270);
		Chromaticities[1] = FVector2d(0.170, 1.140);
		Chromaticities[2] = FVector2d(0.080, -0.100);
		Chromaticities[3] = GetWhitePoint(EWhitePoint::CIE1931_D65);
		break;
	case EColorSpace::GoProProtuneNative:
		Chromaticities[0] = FVector2d(0.698448, 0.193026);
		Chromaticities[1] = FVector2d(0.329555, 1.024597);
		Chromaticities[2] = FVector2d(0.108443, -0.034679);
		Chromaticities[3] = GetWhitePoint(EWhitePoint::CIE1931_D65);
		break;
	case EColorSpace::PanasonicVGamut:
		Chromaticities[0] = FVector2d(0.730, 0.280);
		Chromaticities[1] = FVector2d(0.165, 0.840);
		Chromaticities[2] = FVector2d(0.100, -0.030);
		Chromaticities[3] = GetWhitePoint(EWhitePoint::CIE1931_D65);
		break;
	case EColorSpace::PLASA_E1_54:
		Chromaticities[0] = FVector2d(0.7347, 0.2653);
		Chromaticities[1] = FVector2d(0.1596, 0.8404);
		Chromaticities[2] = FVector2d(0.0366, 0.0001);
		Chromaticities[3] = FVector2d(0.4254, 0.4044);
		break;
	case EColorSpace::ACESCAM16:
		Chromaticities[0] = FVector2d(0.8336, 0.1735);
		Chromaticities[1] = FVector2d(2.3854, -1.4659);
		Chromaticities[2] = FVector2d(0.087, -0.125);
		Chromaticities[3] = FVector2d(0.333, 0.333);
		break;

	default:
		checkNoEntry();
		break;
	}

	return Chromaticities;
}

FMatrix44d FColorSpace::CalcRgbToXYZ() const
{
	FMatrix44d Mat = FMatrix44d(
		FVector3d(Chromaticities[0].X, Chromaticities[0].Y, 1.0 - Chromaticities[0].X - Chromaticities[0].Y),
		FVector3d(Chromaticities[1].X, Chromaticities[1].Y, 1.0 - Chromaticities[1].X - Chromaticities[1].Y),
		FVector3d(Chromaticities[2].X, Chromaticities[2].Y, 1.0 - Chromaticities[2].X - Chromaticities[2].Y),
		FVector3d{0,0,0}
	);

	FMatrix44d Inverse = Mat.Inverse();
	const FVector2d& White = Chromaticities[3];
	FVector3d WhiteXYZ = FVector3d(White[0] / White[1], 1.0, (1.0f - White[0] - White[1]) / White[1]);
	FVector3d Scale = Inverse.TransformVector(WhiteXYZ);

	for (unsigned i = 0; i < 3; i++)
	{
		Mat.M[0][i] *= Scale[0];
		Mat.M[1][i] *= Scale[1];
		Mat.M[2][i] *= Scale[2];
	}
	return Mat;
}

bool FColorSpace::Equals(const FColorSpace& ColorSpace, double Tolerance) const
{
	return Equals(ColorSpace.Chromaticities, Tolerance);
}

bool FColorSpace::Equals(const TStaticArray<FVector2d, 4>& InChromaticities, double Tolerance) const
{
	return Chromaticities[0].Equals(InChromaticities[0], Tolerance) &&
		Chromaticities[1].Equals(InChromaticities[1], Tolerance) &&
		Chromaticities[2].Equals(InChromaticities[2], Tolerance) &&
		Chromaticities[3].Equals(InChromaticities[3], Tolerance);
}

bool FColorSpace::IsSRGB() const
{
	return bIsSRGB;
}

FLinearColor FColorSpace::MakeFromColorTemperature(float Temp) const
{
	Temp = FMath::Clamp(Temp, 1000.0f, 15000.0f);

	// Approximate Planckian locus in CIE 1960 UCS
	float u = (0.860117757f + 1.54118254e-4f * Temp + 1.28641212e-7f * Temp * Temp) / (1.0f + 8.42420235e-4f * Temp + 7.08145163e-7f * Temp * Temp);
	float v = (0.317398726f + 4.22806245e-5f * Temp + 4.20481691e-8f * Temp * Temp) / (1.0f - 2.89741816e-5f * Temp + 1.61456053e-7f * Temp * Temp);

	float x = 3.0f * u / (2.0f * u - 8.0f * v + 4.0f);
	float y = 2.0f * v / (2.0f * u - 8.0f * v + 4.0f);
	float z = 1.0f - x - y;

	FVector3d XYZ = FVector3d(1.0 / y * x, 1.0, 1.0 / y * z);
	FVector4d RGB = XYZToRgb.TransformVector(XYZ);

	// The XYZ to RGB transform can result in negative values, so we need to clamp here.
	return FLinearColor(FMath::Max(0.0f, (float)RGB.X), FMath::Max(0.0f, (float)RGB.Y), FMath::Max(0.0f, (float)RGB.Z));
}

float FColorSpace::GetLuminance(const FLinearColor& Color) const
{
	//Note: Equivalent to the dot product of Color and RgbToXYZ.GetColumn(1).
	return Color.R * (float)RgbToXYZ.M[0][1] + Color.G * (float)RgbToXYZ.M[1][1] + Color.B * (float)RgbToXYZ.M[2][1];
}

FLinearColor FColorSpace::GetLuminanceFactors() const
{
	return FLinearColor(RgbToXYZ.GetColumn(1));
}


FMatrix44d FColorSpaceTransform::CalcChromaticAdaptionMatrix(FVector3d SourceXYZ, FVector3d TargetXYZ, EChromaticAdaptationMethod Method)
{
	FMatrix44d XyzToRgb;

	if (Method == EChromaticAdaptationMethod::CAT02)
	{
		XyzToRgb = FMatrix44d(
			FPlane4d(0.7328,  0.4296, -0.1624,  0.),
			FPlane4d(-0.7036,  1.6975,  0.0061,  0.),
			FPlane4d(0.0030,  0.0136,  0.9834,  0.),
			FPlane4d(0., 0., 0., 1.)
		).GetTransposed();
	}
	else if (Method == EChromaticAdaptationMethod::Bradford)
	{
		XyzToRgb = FMatrix44d(
			FPlane4d(0.8951,  0.2664, -0.1614,  0.),
			FPlane4d(-0.7502,  1.7135,  0.0367,  0.),
			FPlane4d(0.0389, -0.0685,  1.0296,  0.),
			FPlane4d(0., 0., 0., 1.)
		).GetTransposed();
	}
	else
	{
		return FMatrix44d::Identity;
	}

	FVector4d SourceRGB = XyzToRgb.TransformVector(SourceXYZ);
	FVector4d TargetRGB = XyzToRgb.TransformVector(TargetXYZ);
	
	FVector4d Scale = FVector4d(
		TargetRGB.X / SourceRGB.X,
		TargetRGB.Y / SourceRGB.Y,
		TargetRGB.Z / SourceRGB.Z,
		1.0);

	FMatrix44d ScaleMat = FMatrix44d::Identity;
	ScaleMat.M[0][0] = Scale.X;
	ScaleMat.M[1][1] = Scale.Y;
	ScaleMat.M[2][2] = Scale.Z;
	ScaleMat.M[3][3] = Scale.W;

	FMatrix44d RgbToXyz = XyzToRgb.Inverse();

	return XyzToRgb * ScaleMat * RgbToXyz;
}

const FColorSpaceTransform& FColorSpaceTransform::GetSRGBToWorkingColorSpace()
{
	// Note: An editor restart is needed for the working color space shader compiler definitions (WORKING_COLOR_SPACE_RGB_TO_XYZ_MAT, XYZ_TO_RGB_WORKING_COLOR_SPACE_MAT, etc), so we can currently rely on the same assumption here.
	static FColorSpaceTransform CachedTransform = FColorSpaceTransform(FColorSpace::GetSRGB(), FColorSpace::GetWorking());
	
	return CachedTransform;
}

const FColorSpaceTransform& FColorSpaceTransform::GetWorkingColorSpaceToSRGB()
{
	// Note: An editor restart is needed for the working color space shader compiler definitions (WORKING_COLOR_SPACE_RGB_TO_XYZ_MAT, XYZ_TO_RGB_WORKING_COLOR_SPACE_MAT, etc), so we can currently rely on the same assumption here.
	static FColorSpaceTransform CachedTransform = FColorSpaceTransform(FColorSpace::GetWorking(), FColorSpace::GetSRGB());

	return CachedTransform;
}

static FMatrix44d CalcColorSpaceTransformMatrix(const FColorSpace& Src, const FColorSpace& Dst, EChromaticAdaptationMethod Method)
{
	if (Method == UE::Color::EChromaticAdaptationMethod::None)
	{
		return Src.GetRgbToXYZ() * Dst.GetXYZToRgb();
	}

	const FVector3d SrcWhite = Src.GetRgbToXYZ().TransformVector(FVector3d::One());
	const FVector3d DstWhite = Dst.GetRgbToXYZ().TransformVector(FVector3d::One());

	if (SrcWhite.Equals(DstWhite, 1.e-7))
	{
		return Src.GetRgbToXYZ() * Dst.GetXYZToRgb();
	}

	FMatrix44d ChromaticAdaptationMat = FColorSpaceTransform::CalcChromaticAdaptionMatrix(SrcWhite, DstWhite, Method);
	return Src.GetRgbToXYZ() * ChromaticAdaptationMat * Dst.GetXYZToRgb();
}

FColorSpaceTransform::FColorSpaceTransform(const FColorSpace& Src, const FColorSpace& Dst, EChromaticAdaptationMethod Method)
	: FMatrix44d(CalcColorSpaceTransformMatrix(Src,Dst,Method))
{

}

FColorSpaceTransform::FColorSpaceTransform(FMatrix44d Matrix)
	: FMatrix44d(MoveTemp(Matrix))
{ }

FLinearColor FColorSpaceTransform::Apply(const FLinearColor& Color) const
{
	FLinearColor Result;
	VectorRegister4Float VecP = VectorLoad(&Color.R);
	VectorRegister4Float VecR = VectorTransformVector(VecP, this);
	VectorStore(VecR, &Result.R);
	return Result;
}


} } // end namespace UE::Color

