// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "CoreTypes.h"
#include "Containers/UnrealString.h"
#include "ColorManagement/TransferFunctions.h"
#include "ColorManagement/ColorSpace.h"
#include "Math/NumericLimits.h"
#include "Math/UnrealMathUtility.h"
#include "Logging/LogMacros.h"
#include "Tests/TestHarnessAdapter.h"

DEFINE_LOG_CATEGORY_STATIC(LogUnrealColorManagementTest, Log, All);

TEST_CASE_NAMED(FTransferFunctionsTest, "System::ColorManagement::TransferFunctions", "[EditorContext][EngineFilter]")
{
	using namespace UE::Color;

	const float TestIncrement = 0.05f;

	// Verify that all transfer functions correctly inverse each other.

	for (uint8 EnumValue = static_cast<uint8>(EEncoding::Linear); EnumValue < static_cast<uint8>(EEncoding::Max); EnumValue++)
	{
		EEncoding EncodingType = static_cast<EEncoding>(EnumValue);

		for (float TestValue = 0.0f; TestValue <= 1.0f; TestValue += TestIncrement)
		{
			float Encoded = UE::Color::Encode(EncodingType, TestValue);
			float Decoded = UE::Color::Decode(EncodingType, Encoded);
			CHECK_MESSAGE(TEXT("Transfer function encode followed by decode must match identity"), FMath::IsNearlyEqual(Decoded, TestValue, UE_KINDA_SMALL_NUMBER));
		}
	}
}

/**
 * Tests if two double matrices (4x4 xyzw) are equal within an optional tolerance
 *
 * @param Mat0 First Matrix
 * @param Mat1 Second Matrix
 * @param Tolerance Error per item allowed for the comparison
 *
 * @return true if equal within tolerance
 */
static bool TestMatricesDoubleEqual(FMatrix44d& Mat0, FMatrix44d& Mat1, double Tolerance = 0.0)
{
	for (int32 Row = 0; Row < 4; ++Row)
	{
		for (int32 Column = 0; Column < 4; ++Column)
		{
			double Diff = Mat0.M[Row][Column] - Mat1.M[Row][Column];
			if (FMath::Abs(Diff) > Tolerance)
			{
				UE_LOGF(LogUnrealColorManagementTest, Log, "Bad(%.8f) at [%i, %i]", Diff, Row, Column);
				return false;
			}
		}
	}
	return true;
}

static void TestRgbToXyzConversionMatrices()
{
	// Note: test matrices generated using the python (colour-science) colour library.

	using namespace UE::Color;

	double Tolerance = 0.000001;

	FMatrix44d Mat0 = FColorSpace(EColorSpace::sRGB).GetRgbToXYZ();
	FMatrix44d Mat1 = FMatrix44d(
		{0.412390799266, 0.357584339384, 0.180480788402, 0.0},
		{0.212639005872, 0.715168678768, 0.0721923153607, 0.0},
		{0.0193308187156, 0.119194779795, 0.95053215225, 0.0},
		{0,0,0,1}
	).GetTransposed();
	CHECK_MESSAGE(TEXT("Rec709 RGB2XYZ Matrix Equality"), TestMatricesDoubleEqual(Mat0, Mat1, Tolerance));

	Mat0 = FColorSpace(EColorSpace::Rec2020).GetRgbToXYZ();
	Mat1 = FMatrix44d(
		{0.636958048301, 0.144616903586, 0.168880975164, 0.0},
		{0.262700212011, 0.677998071519, 0.0593017164699, 0.0},
		{4.99410657447e-17, 0.0280726930491, 1.06098505771, 0.0},
		{0,0,0,1}
	).GetTransposed();
	CHECK_MESSAGE(TEXT("Rec2020 RGB2XYZ Matrix Equality"), TestMatricesDoubleEqual(Mat0, Mat1, Tolerance));

	Mat0 = FColorSpace(EColorSpace::ACESAP0).GetRgbToXYZ();
	Mat1 = FMatrix44d(
		{ 0.9525523959, 0.0, 9.36786e-05, 0.0 },
		{ 0.3439664498, 0.7281660966, -0.0721325464, 0.0 },
		{ 0.0, 0.0, 1.0088251844, 0.0 },
		{ 0,0,0,1 }
	).GetTransposed();
	CHECK_MESSAGE(TEXT("AP0 RGB2XYZ Matrix Equality"), TestMatricesDoubleEqual(Mat0, Mat1, Tolerance));

	Mat0 = FColorSpace(EColorSpace::ACESAP1).GetRgbToXYZ();
	Mat1 = FMatrix44d(
		{ 0.662454181109, 0.134004206456, 0.156187687005, 0.0 },
		{ 0.272228716781, 0.674081765811, 0.0536895174079, 0.0 },
		{ -0.00557464949039, 0.00406073352898, 1.01033910031, 0.0 },
		{ 0,0,0,1 }
	).GetTransposed();
	CHECK_MESSAGE(TEXT("AP1 RGB2XYZ Matrix Equality"), TestMatricesDoubleEqual(Mat0, Mat1, Tolerance));

	Mat0 = FColorSpace(EColorSpace::P3DCI).GetRgbToXYZ();
	Mat1 = FMatrix44d(
		{0.445169815565, 0.277134409207, 0.172282669816, 0.0},
		{0.209491677913, 0.721595254161, 0.0689130679262, 0.0},
		{-3.63410131697e-17, 0.047060560054, 0.907355394362, 0.0},
		{0,0,0,1}
	).GetTransposed();
	CHECK_MESSAGE(TEXT("P3DCI RGB2XYZ Matrix Equality"), TestMatricesDoubleEqual(Mat0, Mat1, Tolerance));

	Mat0 = FColorSpace(EColorSpace::P3D65).GetRgbToXYZ();
	Mat1 = FMatrix44d(
		{0.486570948648, 0.265667693169, 0.198217285234, 0.0},
		{0.22897456407, 0.691738521837, 0.0792869140937, 0.0},
		{-3.97207551693e-17, 0.0451133818589, 1.0439443689, 0.0},
		{0,0,0,1}
	).GetTransposed();
	CHECK_MESSAGE(TEXT("P3D65 RGB2XYZ Matrix Equality"), TestMatricesDoubleEqual(Mat0, Mat1, Tolerance));
}


static void TestColorSpaceTransforms(UE::Color::EChromaticAdaptationMethod Method)
{
	// Note: test matrices generated using the python (colour-science) colour library.

	using namespace UE::Color;

	double Tolerance = UE_SMALL_NUMBER;

	const FColorSpace Src = FColorSpace(EColorSpace::ACESAP1);
	FMatrix44d Mat0, Mat1;

	if (Method == UE::Color::EChromaticAdaptationMethod::None)
	{
		Mat0 = FColorSpaceTransform(Src, FColorSpace(EColorSpace::sRGB), Method);
		Mat1 = FMatrix44d(
			{ 1.73125381945, -0.604043087283, -0.0801077089571, 0.0 },
			{ -0.131618928589, 1.13484150569, -0.00867943255179, 0.0 },
			{ -0.0245682525938, -0.125750404281, 1.06563695775, 0.0 },
			{0,0,0,1}
		).GetTransposed();
		CHECK_MESSAGE(TEXT("AP1->Rec709 without chromatic adaptation"), TestMatricesDoubleEqual(Mat0, Mat1, Tolerance));

		Mat0 = FColorSpaceTransform(Src, FColorSpace(EColorSpace::Rec2020), Method);
		Mat1 = FMatrix44d(
			{ 1.04179138412, -0.0107415627227, -0.00696187506631, 0.0 },
			{ -0.00168312771738, 1.00036605073, -0.0014082109903, 0.0 },
			{ -0.005209686529, -0.0226414456785, 0.95230241486, 0.0 },
			{ 0,0,0,1 }
		).GetTransposed();
		CHECK_MESSAGE(TEXT("AP1->Rec709 without chromatic adaptation"), TestMatricesDoubleEqual(Mat0, Mat1, Tolerance));

		Mat0 = FColorSpaceTransform(Src, FColorSpace(EColorSpace::ACESAP0), Method);
		Mat1 = FMatrix44d(
			{ 0.695452241359, 0.140678696471, 0.163869062214, 0.0 },
			{ 0.0447945633525, 0.859671118443, 0.0955343182103, 0.0 },
			{ -0.00552588255811, 0.00402521030598, 1.00150067225, 0.0 },
			{ 0,0,0,1 }
		).GetTransposed();
		CHECK_MESSAGE(TEXT("AP1->Rec709 without chromatic adaptation"), TestMatricesDoubleEqual(Mat0, Mat1, Tolerance));

		Mat0 = FColorSpaceTransform(Src, FColorSpace(EColorSpace::ACESAP1), Method);
		Mat1 = FMatrix44d::Identity;
		CHECK_MESSAGE(TEXT("AP1->Rec709 without chromatic adaptation"), TestMatricesDoubleEqual(Mat0, Mat1, Tolerance));

		Mat0 = FColorSpaceTransform(Src, FColorSpace(EColorSpace::P3DCI), Method);
		Mat1 = FMatrix44d(
			{ 1.53077277413, -0.322790385146, -0.0736971869439, 0.0 },
			{ -0.0668950445367, 1.03255367118, -0.0105932139741, 0.0 },
			{ -0.0026742897488, -0.0490786970568, 1.11404817691, 0.0 },
			{ 0,0,0,1 }
		).GetTransposed();
		CHECK_MESSAGE(TEXT("AP1->Rec709 without chromatic adaptation"), TestMatricesDoubleEqual(Mat0, Mat1, Tolerance));

		Mat0 = FColorSpaceTransform(Src, FColorSpace(EColorSpace::P3D65), Method);
		Mat1 = FMatrix44d(
			{ 1.40052305923, -0.295324940013, -0.067426473386, 0.0 },
			{ -0.069782360156, 1.07712062473, -0.0110504369624, 0.0 },
			{ -0.0023243874884, -0.0426572735573, 0.968286867584, 0.0 },
			{ 0,0,0,1 }
		).GetTransposed();
		CHECK_MESSAGE(TEXT("AP1->Rec709 without chromatic adaptation"), TestMatricesDoubleEqual(Mat0, Mat1, Tolerance));
	}
	else if (Method == UE::Color::EChromaticAdaptationMethod::Bradford)
	{
		Mat0 = FColorSpaceTransform(Src, FColorSpace(EColorSpace::sRGB), Method);
		Mat1 = FMatrix44d(
			{ 1.70505099266, -0.621792120657, -0.083258872001, 0.0 },
			{ -0.130256417507, 1.14080473658, -0.0105483190684, 0.0 },
			{ -0.0240033568046, -0.128968976065, 1.15297233287, 0.0 },
			{ 0,0,0,1 }
		).GetTransposed();
		CHECK_MESSAGE(TEXT("AP1->Rec709 with Bradford chromatic adaptation"), TestMatricesDoubleEqual(Mat0, Mat1, Tolerance));

		Mat0 = FColorSpaceTransform(Src, FColorSpace(EColorSpace::Rec2020), Method);
		Mat1 = FMatrix44d(
			{ 1.02582474767, -0.0200531908382, -0.0057715568278, 0.0 },
			{ -0.00223436951998, 1.00458650189, -0.0023521323685, 0.0 },
			{ -0.00501335146809, -0.0252900718108, 1.03030342328, 0.0 },
			{ 0,0,0,1 }
		).GetTransposed();
		CHECK_MESSAGE(TEXT("AP1->Rec709 with Bradford chromatic adaptation"), TestMatricesDoubleEqual(Mat0, Mat1, Tolerance));

		Mat0 = FColorSpaceTransform(Src, FColorSpace(EColorSpace::ACESAP0), Method);
		Mat1 = FMatrix44d(
			{ 0.695452241359, 0.140678696471, 0.163869062214, 0.0 },
			{ 0.0447945633525, 0.859671118443, 0.0955343182103, 0.0 },
			{ -0.00552588255811, 0.00402521030598, 1.00150067225, 0.0 },
			{ 0,0,0,1 }
		).GetTransposed();
		CHECK_MESSAGE(TEXT("AP1->Rec709 with Bradford chromatic adaptation"), TestMatricesDoubleEqual(Mat0, Mat1, Tolerance));

		Mat0 = FColorSpaceTransform(Src, FColorSpace(EColorSpace::ACESAP1), Method);
		Mat1 = FMatrix44d::Identity;
		CHECK_MESSAGE(TEXT("AP1->Rec709 with Bradford chromatic adaptation"), TestMatricesDoubleEqual(Mat0, Mat1, Tolerance));

		Mat0 = FColorSpaceTransform(Src, FColorSpace(EColorSpace::P3DCI), Method);
		Mat1 = FMatrix44d(
			{ 1.46412016696, -0.393327041647, -0.0707931253151, 0.0 },
			{ -0.0664765138416, 1.07529152526, -0.00881501141699, 0.0 },
			{ -0.00255286167095, -0.0470296027287, 1.0495824644, 0.0 },
			{ 0,0,0,1 }
		).GetTransposed();
		CHECK_MESSAGE(TEXT("AP1->Rec709 with Bradford chromatic adaptation"), TestMatricesDoubleEqual(Mat0, Mat1, Tolerance));

		Mat0 = FColorSpaceTransform(Src, FColorSpace(EColorSpace::P3D65), Method);
		Mat1 = FMatrix44d(
			{ 1.37921412825, -0.308864144674, -0.0703499835796, 0.0 },
			{ -0.0693348583814, 1.082296746, -0.012961887621, 0.0 },
			{ -0.00215900951357, -0.0454593248373, 1.04761833435, 0.0 },
			{ 0,0,0,1 }
		).GetTransposed();
		CHECK_MESSAGE(TEXT("AP1->Rec709 with Bradford chromatic adaptation"), TestMatricesDoubleEqual(Mat0, Mat1, Tolerance));
	}
}

static void TestAppliedColorTransforms()
{
	using namespace UE::Color;

	// Intentionally misalign the start of FLinearColor member,
	// as a test against 16-byte (128bit) alignment when using 4 float SIMD instructions.
	// 
	// Given that we now load/store from unaligned memory, the above alignment requirement is lifted.
	struct FAlignmentTest
	{
		float Nudge = 0.0f;
		FLinearColor SrcColor = FLinearColor(1.0f, 0.5f, 0.0f);
	};

	FColorSpaceTransform Transform = FColorSpaceTransform(FColorSpace(EColorSpace::sRGB), FColorSpace(EColorSpace::ACESAP1), EChromaticAdaptationMethod::Bradford);

	TUniquePtr<FAlignmentTest> AlignmentTest = MakeUnique<FAlignmentTest>();
	FLinearColor ExpectedResult = FLinearColor(0.78285898f, 0.52837066f, 0.07540048f);
	FLinearColor Result = Transform.Apply(AlignmentTest->SrcColor);

	CHECK_MESSAGE(TEXT("FLinearColor sRGB->AP1 color transform"), Result.Equals(ExpectedResult, UE_SMALL_NUMBER));
}

static void TestLuminance()
{
	using namespace UE::Color;

	// Note: test factors from the python (colour-science) colour library.

	FLinearColor LuminanceFactors = FColorSpace(EColorSpace::sRGB).GetLuminanceFactors();
	CHECK_MESSAGE(TEXT("sRGB luminance factors equality test"), LuminanceFactors.Equals(FLinearColor(0.212639005872f, 0.715168678768f, 0.0721923153607f), UE_SMALL_NUMBER));

	LuminanceFactors = FColorSpace(EColorSpace::ACESAP1).GetLuminanceFactors();
	CHECK_MESSAGE(TEXT("ACESAP1 luminance factors equality test"), LuminanceFactors.Equals(FLinearColor(0.272228716781f, 0.674081765811f, 0.0536895174079f), UE_SMALL_NUMBER));

	LuminanceFactors = FColorSpace(EColorSpace::Rec2020).GetLuminanceFactors();
	CHECK_MESSAGE(TEXT("Rec2020 luminance factors equality test"), LuminanceFactors.Equals(FLinearColor(0.262700212011f, 0.677998071519f, 0.0593017164699f), UE_SMALL_NUMBER));
}

TEST_CASE_NAMED(FHLG2100TransferFunctionsTest, "System::ColorManagement::HLG2100TransferFunctions", "[EditorContext][EngineFilter]")
{
	using namespace UE::Color;

	SECTION("OETF piecewise boundary")
	{
		// Verify the piecewise boundary maps correctly: E = 1/12 <-> E' = 0.5
		CHECK_MESSAGE(TEXT("HLG OETF at boundary (1/12) should equal 0.5"), FMath::IsNearlyEqual(EncodeHLG(1.0f / 12.0f), 0.5f, UE_SMALL_NUMBER));
		CHECK_MESSAGE(TEXT("HLG inverse OETF at boundary (0.5) should equal 1/12"), FMath::IsNearlyEqual(DecodeHLG(0.5f), 1.0f / 12.0f, UE_SMALL_NUMBER));
	}

	SECTION("OETF negative values")
	{
		// HLG OETF should handle negative values via CopySign
		float Positive = EncodeHLG(0.5f);
		float Negative = EncodeHLG(-0.5f);
		CHECK_MESSAGE(TEXT("HLG OETF negative value should mirror positive"), FMath::IsNearlyEqual(Negative, -Positive, UE_SMALL_NUMBER));
	}

	SECTION("Reference White scaling via GetReferenceWhiteLinearScale")
	{
		// BT.2408 + HLG decode: 75% HLG signal (BT.2100 linear = DecodeHLG(0.75)) should map to UE scene-linear 1.0.
		const float HLGDecodeScale = GetReferenceWhiteLinearScale(EEncoding::HLG, EReferenceWhite::BT2408, EReferenceWhiteDirection::Decode);
		CHECK_MESSAGE(TEXT("HLG + BT2408 decode-scale times DecodeHLG(0.75) should equal 1.0"),
			FMath::IsNearlyEqual(HLGDecodeScale * DecodeHLG(0.75f), 1.0f, 1e-4f));
		// The encode scale is the inverse.
		const float HLGEncodeScale = GetReferenceWhiteLinearScale(EEncoding::HLG, EReferenceWhite::BT2408, EReferenceWhiteDirection::Encode);
		CHECK_MESSAGE(TEXT("HLG + BT2408 encode-scale should equal 1 / decode-scale"),
			FMath::IsNearlyEqual(HLGEncodeScale, 1.0f / HLGDecodeScale, 1e-4f));
		// BT.2408 + PQ: 203 nits <-> UE scene-linear 1.0.
		CHECK_MESSAGE(TEXT("ST2084 + BT2408 decode-scale should equal 1/203"),
			FMath::IsNearlyEqual(GetReferenceWhiteLinearScale(EEncoding::ST2084, EReferenceWhite::BT2408, EReferenceWhiteDirection::Decode), 1.0f / 203.0f, 1e-6f));
		CHECK_MESSAGE(TEXT("ST2084 + BT2408 encode-scale should equal 203"),
			FMath::IsNearlyEqual(GetReferenceWhiteLinearScale(EEncoding::ST2084, EReferenceWhite::BT2408, EReferenceWhiteDirection::Encode), 203.0f, 1e-6f));
		// None passes through unchanged in either direction.
		CHECK_MESSAGE(TEXT("None + ST2084 decode-scale should equal 1.0"),
			FMath::IsNearlyEqual(GetReferenceWhiteLinearScale(EEncoding::ST2084, EReferenceWhite::None, EReferenceWhiteDirection::Decode), 1.0f, 1e-6f));
		CHECK_MESSAGE(TEXT("None + HLG encode-scale should equal 1.0"),
			FMath::IsNearlyEqual(GetReferenceWhiteLinearScale(EEncoding::HLG, EReferenceWhite::None, EReferenceWhiteDirection::Encode), 1.0f, 1e-6f));
		// BT1886 reproduces the pre-BT.2408 media pipeline convention.
		CHECK_MESSAGE(TEXT("BT1886 + ST2084 decode-scale should equal 1/100"),
			FMath::IsNearlyEqual(GetReferenceWhiteLinearScale(EEncoding::ST2084, EReferenceWhite::BT1886, EReferenceWhiteDirection::Decode), 1.0f / 100.0f, 1e-6f));
		// BT1886 + HLG uses an in-house 100-nit convention: BT.2100 linear = (100/1000)^(1/1.2).
		const float ExpectedBT1886HlgEncodeScale = FMath::Pow(100.0f / 1000.0f, 1.0f / 1.2f);
		CHECK_MESSAGE(TEXT("BT1886 + HLG encode-scale should equal (100/1000)^(1/1.2)"),
			FMath::IsNearlyEqual(GetReferenceWhiteLinearScale(EEncoding::HLG, EReferenceWhite::BT1886, EReferenceWhiteDirection::Encode), ExpectedBT1886HlgEncodeScale, 1e-6f));
		CHECK_MESSAGE(TEXT("BT1886 + HLG decode-scale should equal its inverse"),
			FMath::IsNearlyEqual(GetReferenceWhiteLinearScale(EEncoding::HLG, EReferenceWhite::BT1886, EReferenceWhiteDirection::Decode), 1.0f / ExpectedBT1886HlgEncodeScale, 1e-4f));
		CHECK_MESSAGE(TEXT("BT1886 + ST2084 encode-scale should equal 100"),
			FMath::IsNearlyEqual(GetReferenceWhiteLinearScale(EEncoding::ST2084, EReferenceWhite::BT1886, EReferenceWhiteDirection::Encode), 100.0f, 1e-6f));
		// DisableNormalization always resolves to 1.0 regardless of encoding or direction.
		CHECK_MESSAGE(TEXT("DisableNormalization + ST2084 decode-scale should equal 1.0"),
			FMath::IsNearlyEqual(GetReferenceWhiteLinearScale(EEncoding::ST2084, EReferenceWhite::DisableNormalization, EReferenceWhiteDirection::Decode), 1.0f, 1e-6f));
		CHECK_MESSAGE(TEXT("DisableNormalization + HLG encode-scale should equal 1.0"),
			FMath::IsNearlyEqual(GetReferenceWhiteLinearScale(EEncoding::HLG, EReferenceWhite::DisableNormalization, EReferenceWhiteDirection::Encode), 1.0f, 1e-6f));
		CHECK_MESSAGE(TEXT("DisableNormalization + Linear decode-scale should equal 1.0"),
			FMath::IsNearlyEqual(GetReferenceWhiteLinearScale(EEncoding::Linear, EReferenceWhite::DisableNormalization, EReferenceWhiteDirection::Decode), 1.0f, 1e-6f));
	}

	SECTION("BT.2100 round-trip at default luminance")
	{
		// Encode then decode should produce identity for display-referred colors.
		// Note: black and single-channel colors are excluded because the OOTF luminance weighting
		// (Ys^(gamma-1)) is degenerate at zero luminance, making round-trip undefined.
		const FLinearColor TestColors[] = {
			FLinearColor(100.0f, 200.0f, 50.0f),
			FLinearColor(500.0f, 500.0f, 500.0f),
			FLinearColor(1000.0f, 1000.0f, 1000.0f),
			FLinearColor(10.0f, 50.0f, 30.0f),
		};

		for (const FLinearColor& Src : TestColors)
		{
			FLinearColor Encoded = EncodeHLG_2100(Src);
			FLinearColor Decoded = DecodeHLG_2100(Encoded);

			bool bRClose = FMath::IsNearlyEqual(Decoded.R, Src.R, FMath::Abs(Src.R) * UE_KINDA_SMALL_NUMBER);
			bool bGClose = FMath::IsNearlyEqual(Decoded.G, Src.G, FMath::Abs(Src.G) * UE_KINDA_SMALL_NUMBER);
			bool bBClose = FMath::IsNearlyEqual(Decoded.B, Src.B, FMath::Abs(Src.B) * UE_KINDA_SMALL_NUMBER);
			CHECK_MESSAGE(TEXT("BT.2100 round-trip should preserve color values"), bRClose && bGClose && bBClose);
		}
	}

	SECTION("BT.2100 round-trip at various peak luminances")
	{
		const FLinearColor Src(200.0f, 400.0f, 100.0f);
		const float PeakLuminances[] = { 400.0f, 1000.0f, 2000.0f, 4000.0f };

		for (float Peak : PeakLuminances)
		{
			FLinearColor Encoded = EncodeHLG_2100(Src, Peak);
			FLinearColor Decoded = DecodeHLG_2100(Encoded, Peak);

			bool bRClose = FMath::IsNearlyEqual(Decoded.R, Src.R, FMath::Abs(Src.R) * UE_KINDA_SMALL_NUMBER);
			bool bGClose = FMath::IsNearlyEqual(Decoded.G, Src.G, FMath::Abs(Src.G) * UE_KINDA_SMALL_NUMBER);
			bool bBClose = FMath::IsNearlyEqual(Decoded.B, Src.B, FMath::Abs(Src.B) * UE_KINDA_SMALL_NUMBER);
			CHECK_MESSAGE(TEXT("BT.2100 round-trip at varying peak luminance should preserve color values"), bRClose && bGClose && bBClose);
		}
	}

	SECTION("BT.2100 round-trip with black luminance")
	{
		const FLinearColor Src(300.0f, 600.0f, 150.0f);
		const float PeakLuminance = 1000.0f;
		const float BlackLuminance = 0.01f;

		FLinearColor Encoded = EncodeHLG_2100(Src, PeakLuminance, BlackLuminance);
		FLinearColor Decoded = DecodeHLG_2100(Encoded, PeakLuminance, BlackLuminance);

		bool bRClose = FMath::IsNearlyEqual(Decoded.R, Src.R, FMath::Abs(Src.R) * UE_KINDA_SMALL_NUMBER);
		bool bGClose = FMath::IsNearlyEqual(Decoded.G, Src.G, FMath::Abs(Src.G) * UE_KINDA_SMALL_NUMBER);
		bool bBClose = FMath::IsNearlyEqual(Decoded.B, Src.B, FMath::Abs(Src.B) * UE_KINDA_SMALL_NUMBER);
		CHECK_MESSAGE(TEXT("BT.2100 round-trip with black luminance should preserve color values"), bRClose && bGClose && bBClose);
	}

	SECTION("BT.2100 black input does not produce NaN at low peak luminance")
	{
		// PeakLuminance below ~302 cd/m^2 yields Gamma < 1 via HLG_SystemGamma_Extended,
		// making the OOTF exponent (Gamma-1) negative. Without a zero-luminance guard,
		// pow(0, negative) = +inf and inf * 0 = NaN.
		const FLinearColor Black(0.0f, 0.0f, 0.0f, 1.0f);
		const float LowPeakLuminances[] = { 100.0f, 200.0f, 300.0f };

		for (float Peak : LowPeakLuminances)
		{
			FLinearColor Encoded = EncodeHLG_2100(Black, Peak);
			CHECK_MESSAGE(TEXT("Encoding black at low peak luminance must not produce NaN"),
				!FMath::IsNaN(Encoded.R) && !FMath::IsNaN(Encoded.G) && !FMath::IsNaN(Encoded.B));

			FLinearColor Decoded = DecodeHLG_2100(Encoded, Peak);
			CHECK_MESSAGE(TEXT("Decoding black at low peak luminance must not produce NaN"),
				!FMath::IsNaN(Decoded.R) && !FMath::IsNaN(Decoded.G) && !FMath::IsNaN(Decoded.B));

			CHECK_MESSAGE(TEXT("Black should round-trip to black RGB"),
				FMath::IsNearlyZero(Decoded.R) && FMath::IsNearlyZero(Decoded.G) && FMath::IsNearlyZero(Decoded.B));

			CHECK_MESSAGE(TEXT("Alpha should be preserved through BT.2100 round-trip"),
				FMath::IsNearlyEqual(Decoded.A, 1.0f, UE_SMALL_NUMBER));
		}
	}

}

TEST_CASE_NAMED(FColorSpaceTest, "System::ColorManagement::ColorSpace", "[EditorContext][EngineFilter]")
{
	using namespace UE::Color;

	FColorSpace CS = FColorSpace(EColorSpace::sRGB);
	FMatrix44d Mat0 = FColorSpaceTransform(CS, CS, EChromaticAdaptationMethod::None);
	FMatrix44d Mat1 = FMatrix44d::Identity;
	CHECK_MESSAGE(TEXT("Identity color space conversion to itself should match identity"), TestMatricesDoubleEqual(Mat0, Mat1, 0.00000001));

	SECTION("RgbToXyzConversionMatrices")
	TestRgbToXyzConversionMatrices();

	SECTION("ColorSpaceTransforms")
	TestColorSpaceTransforms(EChromaticAdaptationMethod::None);

	SECTION("ColorSpaceTransforms_Bradford")
	TestColorSpaceTransforms(EChromaticAdaptationMethod::Bradford);

	SECTION("AppliedColorTransforms")
	TestAppliedColorTransforms();

	SECTION("Luminance")
	TestLuminance();
}

#endif //WITH_TESTS
