// Copyright Epic Games, Inc. All Rights Reserved.

#include "ColorManagement/TransferFunctions.h"
#include "ColorManagement/ColorSpace.h"
#include "Logging/LogMacros.h"

DEFINE_LOG_CATEGORY_STATIC(LogTransferFunctions, Log, All);

namespace UE { namespace Color {

inline TFunction<float(float)> GetTransferFunction(EEncoding SourceEncoding, bool bIsEncode)
{
	switch (SourceEncoding)
	{
	case EEncoding::None:        return &Linear;
	case EEncoding::Linear:      return &Linear;
	case EEncoding::sRGB:        return bIsEncode ? &EncodeSRGB : &DecodeSRGB;
	case EEncoding::ST2084:      return bIsEncode ? &EncodeST2084 : &DecodeST2084;
	case EEncoding::Gamma22:     return bIsEncode ? &EncodeGamma22 : &DecodeGamma22;
	case EEncoding::BT1886:      return bIsEncode ? &EncodeBT1886 : &DecodeBT1886;
	case EEncoding::Gamma26:     return bIsEncode ? &EncodeGamma26 : &DecodeGamma26;
	case EEncoding::Cineon:      return bIsEncode ? &EncodeCineon : &DecodeCineon;
	case EEncoding::REDLog:      return bIsEncode ? &EncodeREDLog : &DecodeREDLog;
	case EEncoding::REDLog3G10:  return bIsEncode ? &EncodeREDLog3G10 : &DecodeREDLog3G10;
	case EEncoding::SLog1:       return bIsEncode ? &EncodeSLog1 : &DecodeSLog1;
	case EEncoding::SLog2:       return bIsEncode ? &EncodeSLog2 : &DecodeSLog2;
	case EEncoding::SLog3:       return bIsEncode ? &EncodeSLog3 : &DecodeSLog3;
	case EEncoding::AlexaV3LogC: return bIsEncode ? &EncodeArriAlexaV3LogC : &DecodeArriAlexaV3LogC;
	case EEncoding::CanonLog:    return bIsEncode ? &EncodeCanonLog : &DecodeCanonLog;
	case EEncoding::ProTune:     return bIsEncode ? &EncodeGoProProTune : &DecodeGoProProTune;
	case EEncoding::VLog:        return bIsEncode ? &EncodePanasonicVLog : &DecodePanasonicVLog;
	case EEncoding::HLG:         return bIsEncode ? &EncodeHLG : &DecodeHLG;
	default:
		check(false);
		break;
	}

	UE_LOGF(LogTransferFunctions, Warning, "Failed to find valid transfer function for enum value %d.", int(SourceEncoding));

	return nullptr;
}

namespace
{

float HLG_SystemGamma(const float PeakLuminance)
{
	return 1.2f + 0.42f * FMath::LogX(10.0f, PeakLuminance / 1000.0f);
}

float HLG_SystemGamma_Extended(const float PeakLuminance)
{
	// See ITU-R BT.2100-3 Note 5f: adjusted formula for displays with a nominal peak luminance other than 1000 cd/m^2.
	constexpr float Kappa = 1.111f;
	const float Exponent = FMath::Log2(PeakLuminance / 1000.0f);

	return 1.2f * FMath::Pow(Kappa, Exponent);
}

FLinearColor HLG_OOTF(const FLinearColor& Value, const float PeakLuminance, const float Gamma)
{
	const float Alpha = PeakLuminance;
	const float Ys = FColorSpace::GetRec2020().GetLuminance(Value);

	if (Ys <= 0.0f)
	{
		return FLinearColor(0.0f, 0.0f, 0.0f, Value.A);
	}

	const float Scale = Alpha * FMath::Pow(Ys, Gamma - 1.0f);
	return FLinearColor(Value.R * Scale, Value.G * Scale, Value.B * Scale, Value.A);
}

FLinearColor HLG_InverseOOTF(const FLinearColor& Value, const float PeakLuminance, const float Gamma)
{
	const float Alpha = PeakLuminance;
	const float Yd = FColorSpace::GetRec2020().GetLuminance(Value);

	if (Yd <= 0.0f)
	{
		return FLinearColor(0.0f, 0.0f, 0.0f, Value.A);
	}

	const float Scale = FMath::Pow(Yd / Alpha, (1.0f - Gamma) / Gamma) / Alpha;
	return FLinearColor(Value.R * Scale, Value.G * Scale, Value.B * Scale, Value.A);
}

} // anonymous namespace

FLinearColor EncodeHLG_2100(const FLinearColor& LinearDisplayReferred, const float PeakLuminance, const float BlackLuminance)
{
	if (!ensureMsgf(PeakLuminance > 0.0f, TEXT("Peak luminance should be positive, aborting.")))
	{
		return LinearDisplayReferred;
	}

	if (!ensureMsgf(BlackLuminance >= 0.0f && BlackLuminance < PeakLuminance, TEXT("Black luminance is out of range, aborting.")))
	{
		return LinearDisplayReferred;
	}

	const float Gamma =
		(PeakLuminance < 400.0f || PeakLuminance > 2000.0f)
		? HLG_SystemGamma_Extended(PeakLuminance)
		: HLG_SystemGamma(PeakLuminance);

	float Beta = FMath::Sqrt(3.0f * FMath::Pow(BlackLuminance / PeakLuminance, 1.0f / Gamma));

	// Beta must be in [0, 1) for valid black lift. Values >= 1 arise from non-physical BlackLuminance
	// and would cause division by zero or signal inversion in the affine mapping below.
	if (!ensureMsgf(Beta < 1.0f, TEXT("HLG Beta (%.4f) >= 1 from BlackLuminance=%.2f, PeakLuminance=%.2f. Clamping."
		" Reduce BlackLuminance so that Lb/Lw < (1/3)^gamma."), Beta, BlackLuminance, PeakLuminance))
	{
		Beta = 1.0f - UE_SMALL_NUMBER;
	}

	// The inverse OOTF maps relative display to scene (preserves alpha)
	FLinearColor Output = HLG_InverseOOTF(LinearDisplayReferred, PeakLuminance, Gamma);

	// Encode into HLG signal using OETF
	Output.R = EncodeHLG(Output.R);
	Output.G = EncodeHLG(Output.G);
	Output.B = EncodeHLG(Output.B);

	// Invert user black lift (RGB only, preserve alpha)
	const float InvBetaRange = 1.0f / (1.0f - Beta);
	Output.R = (Output.R - Beta) * InvBetaRange;
	Output.G = (Output.G - Beta) * InvBetaRange;
	Output.B = (Output.B - Beta) * InvBetaRange;

	return Output;
}

FLinearColor DecodeHLG_2100(const FLinearColor& EncodedSceneReferred, const float PeakLuminance, const float BlackLuminance)
{
	if (!ensureMsgf(PeakLuminance > 0.0f, TEXT("Peak luminance should be positive, aborting.")))
	{
		return EncodedSceneReferred;
	}

	if (!ensureMsgf(BlackLuminance >= 0.0f && BlackLuminance < PeakLuminance, TEXT("Black luminance is out of range, aborting.")))
	{
		return EncodedSceneReferred;
	}

	const float Gamma =
		(PeakLuminance < 400.0f || PeakLuminance > 2000.0f)
		? HLG_SystemGamma_Extended(PeakLuminance)
		: HLG_SystemGamma(PeakLuminance);

	float Beta = FMath::Sqrt(3.0f * FMath::Pow(BlackLuminance / PeakLuminance, 1.0f / Gamma));

	// Beta must be in [0, 1) for valid black lift. Values >= 1 arise from non-physical BlackLuminance
	// and would cause signal inversion in the affine mapping below.
	if (!ensureMsgf(Beta < 1.0f, TEXT("HLG Beta (%.4f) >= 1 from BlackLuminance=%.2f, PeakLuminance=%.2f. Clamping."
		" Reduce BlackLuminance so that Lb/Lw < (1/3)^gamma."), Beta, BlackLuminance, PeakLuminance))
	{
		Beta = 1.0f - UE_SMALL_NUMBER;
	}

	// Apply user black lift, clamping to zero per BT.2100-3 EOTF: max(0, (1-beta)*E' + beta)
	// Applied to RGB only to preserve alpha.
	const float BetaRange = 1.0f - Beta;
	FLinearColor Output = EncodedSceneReferred;
	Output.R = FMath::Max(0.0f, BetaRange * Output.R + Beta);
	Output.G = FMath::Max(0.0f, BetaRange * Output.G + Beta);
	Output.B = FMath::Max(0.0f, BetaRange * Output.B + Beta);

	// Decode HLG signal using inverse OETF
	Output.R = DecodeHLG(Output.R);
	Output.G = DecodeHLG(Output.G);
	Output.B = DecodeHLG(Output.B);

	// The OOTF maps relative scene to relative display
	return HLG_OOTF(Output, PeakLuminance, Gamma);
}

TFunction<float(float)> GetEncodeFunction(EEncoding SourceEncoding)
{
	return GetTransferFunction(SourceEncoding, true);
}

TFunction<float(float)> GetDecodeFunction(EEncoding SourceEncoding)
{
	return GetTransferFunction(SourceEncoding, false);
}

float Encode(EEncoding SourceEncoding, float Value)
{
	if (TFunction<float(float)> TransferFn = GetEncodeFunction(SourceEncoding))
	{
		return TransferFn(Value);
	}
	else
	{
		return Value;
	}
}

float Decode(EEncoding SourceEncoding, float Value)
{
	if (TFunction<float(float)> TransferFn = GetDecodeFunction(SourceEncoding))
	{
		return TransferFn(Value);
	}
	else
	{
		return Value;
	}
}

TFunction<FLinearColor(const FLinearColor&)> GetColorEncodeFunction(EEncoding SourceEncoding)
{
	TFunction<float(float)> TransferFn = GetEncodeFunction(SourceEncoding);

	if (!TransferFn)
	{
		return nullptr;
	}

	return [TransferFn](const FLinearColor& Color) -> FLinearColor
	{
		return FLinearColor(TransferFn(Color.R), TransferFn(Color.G), TransferFn(Color.B), Color.A);
	};
}

TFunction<FLinearColor(const FLinearColor&)> GetColorDecodeFunction(EEncoding SourceEncoding)
{
	TFunction<float(float)> TransferFn = GetDecodeFunction(SourceEncoding);

	if (!TransferFn)
	{
		return nullptr;
	}

	return [TransferFn](const FLinearColor& Color) -> FLinearColor
	{
		return FLinearColor(TransferFn(Color.R), TransferFn(Color.G), TransferFn(Color.B), Color.A);
	};
}

FLinearColor Encode(EEncoding SourceEncoding, const FLinearColor& Color)
{
	if (TFunction<FLinearColor(const FLinearColor&)> TransferFn = GetColorEncodeFunction(SourceEncoding))
	{
		return TransferFn(Color);
	}
	else
	{
		return Color;
	}
}

FLinearColor Decode(EEncoding SourceEncoding, const FLinearColor& Color)
{
	if (TFunction<FLinearColor(const FLinearColor&)> TransferFn = GetColorDecodeFunction(SourceEncoding))
	{
		return TransferFn(Color);
	}
	else
	{
		return Color;
	}
}

float GetReferenceWhiteLinearScale(EEncoding Encoding, EReferenceWhite ReferenceWhite, EReferenceWhiteDirection Direction)
{
	if (ReferenceWhite == EReferenceWhite::None)
	{
		return 1.0f;
	}

	// Compute the encode-direction scale (UE scene-linear -> transfer-function linear).
	// The decode direction is its inverse.
	float EncodeScale = 1.0f;
	switch (ReferenceWhite)
	{
	case EReferenceWhite::BT2408:
		switch (Encoding)
		{
		case EEncoding::ST2084:
			// PQ EOTF takes absolute cd/m2; BT.2408 diffuse white is 203 nits.
			EncodeScale = 203.0f;
			break;
		case EEncoding::HLG:
		{
			// HLG OETF takes BT.2100 scene-linear [0, 1]; BT.2408 diffuse white lands
			// at 75% HLG signal, i.e. DecodeHLG(0.75) (~0.265) in BT.2100 linear.
			// Scaling UE scene-linear 1.0 by DecodeHLG(0.75) maps diffuse white to
			// the 75% HLG signal on encode. See ITU-R BT.2408 table 1. Cached
			// on first call so the log/exp in DecodeHLG does not run per frame.
			static const float Scale = DecodeHLG(0.75f);
			EncodeScale = Scale;
			break;
		}
		default:
			return 1.0f;
		}
		break;
	case EReferenceWhite::BT1886:
		// ITU-R BT.1886 SDR reference white (100 cd/m2).
		switch (Encoding)
		{
		case EEncoding::ST2084:
			EncodeScale = 100.0f;
			break;
		case EEncoding::HLG:
		{
			// In-house convention: mirror BT.2408's formula structure but target 100 cd/m2
			// paper white instead of 203 cd/m2 on a 1000 cd/m2 BT.2100 reference display.
			// BT.2100 linear = (display_nits / peak_nits)^(1/gamma), with gamma = 1.2 and
			// peak = 1000 cd/m2. (0.1)^(1/1.2) ~= 0.1468; cf. DecodeHLG(0.75) ~= (0.203)^(1/1.2)
			// ~= 0.265 for BT.2408. Cached on first call.
			static const float Scale = FMath::Pow(100.0f / 1000.0f, 1.0f / 1.2f);
			EncodeScale = Scale;
			break;
		}
		default:
			return 1.0f;
		}
		break;
	case EReferenceWhite::DisableNormalization:
		return 1.0f;
	default:
		return 1.0f;
	}

	return (Direction == EReferenceWhiteDirection::Decode) ? (1.0f / EncodeScale) : EncodeScale;
}

} } // end namespace UE::Color
