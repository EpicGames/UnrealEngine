// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compression/CompressedBuffer.h"

#include "Algo/Accumulate.h"
#include "Algo/ForEach.h"
#include "Async/ParallelFor.h"
#include "Compression/OodleDataCompression.h"
#include "Containers/ArrayView.h"
#include "HAL/PlatformMisc.h"
#include "Hash/Blake3.h"
#include "IO/IoHash.h"
#include "Math/UnrealMathUtility.h"
#include "Memory/MemoryView.h"
#include "Misc/ByteSwap.h"
#include "Misc/Crc.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "Serialization/Archive.h"

THIRD_PARTY_INCLUDES_START
#include "Compression/lz4.h"
THIRD_PARTY_INCLUDES_END

namespace UE::CompressedBuffer::Private
{

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

static constexpr uint64 DefaultBlockSize = 256 * 1024;
static constexpr uint64 DefaultHeaderSize = 4 * 1024;

static constexpr uint64 ParallelDecodeMinRawSize = 4 * 1024 * 1024;

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/** Method used to compress the data in a compressed buffer. */
enum class EMethod : uint8
{
	/** Header is followed by one uncompressed block. */
	None = 0,
	/** Header is followed by an array of compressed block sizes then the compressed blocks. */
	Oodle = 3,
	/** Header is followed by an array of compressed block sizes then the compressed blocks. */
	LZ4 = 4,
};

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/** Header used on every compressed buffer. Always stored in big-endian format. */
struct FHeader
{
	static constexpr uint32 ExpectedMagic = 0xb7756362; // <dot>ucb

	/** A magic number to identify a compressed buffer. Always 0xb7756362. */
	uint32 Magic = ExpectedMagic;
	/** A CRC-32 used to check integrity of the buffer. Uses the polynomial 0x04c11db7. */
	uint32 Crc32 = 0;
	/** The method used to compress the buffer. Affects layout of data following the header. */
	EMethod Method = EMethod::None;
	/** The method-specific compressor used to compress the buffer. */
	uint8 Compressor = 0;
	/** The method-specific compression level used to compress the buffer. */
	uint8 CompressionLevel = 0;
	/** The power of two size of every uncompressed block except the last. Size is 1 << BlockSizeExponent. */
	uint8 BlockSizeExponent = 0;
	/** The number of blocks that follow the header. */
	uint32 BlockCount = 0;
	/** The total size of the uncompressed data. */
	uint64 TotalRawSize = 0;
	/** The total size of the compressed data including the header. */
	uint64 TotalCompressedSize = 0;
	/** The hash of the uncompressed data. */
	FBlake3Hash RawHash;

	/** Checks validity of the buffer based on the magic number, method, and CRC-32. */
	static bool IsValid(const FCompositeBuffer& CompressedData);
	static bool IsValid(const FSharedBuffer& CompressedData) { return IsValid(FCompositeBuffer(CompressedData)); }

	/** Read a header from a buffer that is at least sizeof(FHeader) without any validation. */
	static FHeader Read(const FCompositeBuffer& CompressedData)
	{
		FHeader Header;
		if (sizeof(FHeader) <= CompressedData.GetSize())
		{
			CompressedData.CopyTo(MakeMemoryView(&Header, &Header + 1));
			Header.ByteSwap();
		}
		return Header;
	}

	/**
	 * Write a header to a memory view that is at least sizeof(FHeader).
	 *
	 * @param HeaderView   View of the header to write, including any method-specific header data.
	 */
	void Write(const FMutableMemoryView HeaderView) const
	{
		FHeader Header = *this;
		Header.ByteSwap();
		HeaderView.CopyFrom(MakeMemoryView(&Header, &Header + 1));
		Header.ByteSwap();
		Header.Crc32 = CalculateCrc32(HeaderView);
		Header.ByteSwap();
		HeaderView.CopyFrom(MakeMemoryView(&Header, &Header + 1));
	}

	/** Calculate the CRC-32 from a view of a header including any method-specific header data. */
	static uint32 CalculateCrc32(const FMemoryView HeaderView)
	{
		uint32 Crc32 = 0;
		constexpr uint64 MethodOffset = STRUCT_OFFSET(FHeader, Method);
		for (FMemoryView View = HeaderView + MethodOffset; const uint64 ViewSize = View.GetSize();)
		{
			const int32 Size = int32(FMath::Min<uint64>(ViewSize, MAX_int32));
			Crc32 = FCrc::MemCrc32(View.GetData(), Size, Crc32);
			View += Size;
		}
		return Crc32;
	}

	void ByteSwap()
	{
		Magic = NETWORK_ORDER32(Magic);
		Crc32 = NETWORK_ORDER32(Crc32);
		BlockCount = NETWORK_ORDER32(BlockCount);
		TotalRawSize = NETWORK_ORDER64(TotalRawSize);
		TotalCompressedSize = NETWORK_ORDER64(TotalCompressedSize);
	}

	bool TryGetCompressParameters(
		ECompressedBufferCompressor& OutCompressor,
		ECompressedBufferCompressionLevel& OutCompressionLevel,
		uint64& OutBlockSize) const
	{
		switch (Method)
		{
		case EMethod::None:
			OutCompressor = ECompressedBufferCompressor::NotSet;
			OutCompressionLevel = ECompressedBufferCompressionLevel::None;
			OutBlockSize = 0;
			return true;
		case EMethod::Oodle:
			OutCompressor = ECompressedBufferCompressor(Compressor);
			OutCompressionLevel = ECompressedBufferCompressionLevel(CompressionLevel);
			OutBlockSize = uint64(1) << BlockSizeExponent;
			return true;
		default:
			return false;
		}
	}
};

static_assert(sizeof(FHeader) == 64, "FHeader is the wrong size.");

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

class FDecoderSource
{
public:
	virtual bool SupportsParallelRead() const = 0;
	virtual bool Read(uint64 Offset, FMutableMemoryView Data) const = 0;
	virtual FMemoryView ReadOrView(uint64 Offset, uint64 Size, FDecoderContext& Context) const = 0;
	virtual FCompositeBuffer ReadToComposite(uint64 Offset, uint64 Size) const = 0;
};

class FEncoder
{
public:
	virtual FCompositeBuffer Compress(const FCompositeBuffer& RawData, uint64 BlockSize) const = 0;
};

class FDecoder
{
public:
	virtual uint64 GetHeaderSize(const FHeader& Header) const = 0;
	virtual bool TryDecompressTo(FDecoderContext& Context, const FDecoderSource& Source, const FHeader& Header, FMemoryView HeaderView, uint64 RawOffset, FMutableMemoryView RawView, ECompressedBufferDecompressFlags Flags) const = 0;
	virtual FCompositeBuffer DecompressToComposite(FDecoderContext& Context, const FDecoderSource& Source, const FHeader& Header, const FMemoryView HeaderView, uint64 RawOffset, uint64 RawSize) const = 0;
};

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

class FNoneEncoder final : public FEncoder
{
public:
	FCompositeBuffer Compress(const FCompositeBuffer& RawData, const uint64 BlockSize) const final
	{
		FHeader Header;
		Header.Method = EMethod::None;
		Header.BlockCount = 1;
		Header.TotalRawSize = RawData.GetSize();
		Header.TotalCompressedSize = Header.TotalRawSize + sizeof(FHeader);
		Header.RawHash = FBlake3::HashBuffer(RawData);

		FUniqueBuffer HeaderData = FUniqueBuffer::Alloc(sizeof(FHeader));
		Header.Write(HeaderData);
		return FCompositeBuffer(HeaderData.MoveToShared(), RawData.MakeOwned());
	}
};

class FNoneDecoder final : public FDecoder
{
public:
	uint64 GetHeaderSize(const FHeader& Header) const final
	{
		return sizeof(FHeader);
	}

	bool TryDecompressTo(
		FDecoderContext& Context,
		const FDecoderSource& Source,
		const FHeader& Header,
		const FMemoryView HeaderView,
		const uint64 RawOffset,
		const FMutableMemoryView RawView,
		const ECompressedBufferDecompressFlags Flags) const final
	{
		if (Header.Method == EMethod::None &&
			RawOffset <= Header.TotalRawSize &&
			RawView.GetSize() <= Header.TotalRawSize - RawOffset &&
			Header.TotalCompressedSize == Header.TotalRawSize + sizeof(FHeader))
		{
			return Source.Read(sizeof(FHeader) + RawOffset, RawView);
		}
		return false;
	}

	FCompositeBuffer DecompressToComposite(
		FDecoderContext& Context,
		const FDecoderSource& Source,
		const FHeader& Header,
		const FMemoryView HeaderView,
		const uint64 RawOffset,
		const uint64 RawSize) const final
	{
		if (Header.Method == EMethod::None &&
			RawOffset <= Header.TotalRawSize &&
			RawSize <= Header.TotalRawSize - RawOffset &&
			Header.TotalCompressedSize == Header.TotalRawSize + sizeof(FHeader))
		{
			return Source.ReadToComposite(sizeof(FHeader) + RawOffset, RawSize);
		}
		return FCompositeBuffer();
	}
};

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

class FBlockEncoder : public FEncoder
{
public:
	FCompositeBuffer Compress(const FCompositeBuffer& RawData, uint64 BlockSize) const final;

protected:
	virtual EMethod GetMethod() const = 0;
	virtual uint8 GetCompressor() const = 0;
	virtual uint8 GetCompressionLevel() const = 0;
	virtual uint64 CompressBlockBound(uint64 RawSize) const = 0;
	virtual bool CompressBlock(FMutableMemoryView& CompressedData, FMemoryView RawData) const = 0;

private:
	uint64 GetCompressedBlocksBound(const uint64 BlockCount, const uint64 BlockSize, const uint64 RawSize) const
	{
		switch (BlockCount)
		{
		case 0:  return 0;
		case 1:  return CompressBlockBound(RawSize);
		default: return CompressBlockBound(BlockSize) - BlockSize + RawSize;
		}
	}
};

FCompositeBuffer FBlockEncoder::Compress(const FCompositeBuffer& RawData, const uint64 BlockSize) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FBlockEncoder.Compress);

	checkf(FMath::IsPowerOfTwo(BlockSize) && BlockSize <= MAX_uint32,
		TEXT("BlockSize must be a 32-bit power of two but was %" UINT64_FMT "."), BlockSize);

	const uint64 RawSize = RawData.GetSize();
	if (RawSize == 0)
	{
		return FCompositeBuffer();
	}

	const int64 BlockCount = FMath::DivideAndRoundUp(RawSize, BlockSize);
	check(BlockCount > 0);
	checkf(BlockCount <= MAX_int32, TEXT("Raw data of size %" UINT64_FMT " with block size %" UINT64_FMT " requires "
		"%" UINT64_FMT " blocks, but the limit is %d."), RawSize, BlockSize, BlockCount, MAX_int32);

	FBlake3 RawHash;
	const uint64 MetaSize = sizeof(uint32) * BlockCount;

	// Compress the raw data in blocks and store the raw data for incompressible blocks.
	uint64 CompressedSize = 0;
	FUniqueBuffer CompressedData;
	TArray64<uint32> CompressedBlockSizes;
	TArray<FSharedBuffer> CompressedBlocks;

	struct FBlockContext
	{
		FUniqueBuffer RawBlockCopy;
		FMutableMemoryView CompressedBlocksView;
	};

	// Allocate and encode blocks in parallel with calculating the raw hash.
	CompressedBlockSizes.SetNum(BlockCount);
	TArray<FBlockContext> BlockContexts;
	ParallelForWithPreWorkWithTaskContext(TEXT("BlockEncoder.Compress.PF"), BlockContexts, int32(BlockCount), 1,
		// Executed on the calling thread to allocate the header and optionally the contiguous compressed blocks.
		[this, MetaSize, BlockCount, BlockSize, RawSize, &CompressedData, &CompressedBlocks](const int32 ContextIndex, const int32 ContextCount) -> FBlockContext
		{
			if (ContextCount == 1)
			{
				// Allocate a contiguous buffer for the header, metadata, and compressed blocks.
				const uint64 CompressedDataSize = sizeof(FHeader) + MetaSize + GetCompressedBlocksBound(BlockCount, BlockSize, RawSize);
				CompressedData = FUniqueBuffer::Alloc(CompressedDataSize);
				return {{}, CompressedData.GetView() + sizeof(FHeader) + MetaSize};
			}
			if (ContextIndex == 0)
			{
				// Allocate a buffer for the header and metadata. The compressed blocks are allocated by workers.
				CompressedData = FUniqueBuffer::Alloc(sizeof(FHeader) + MetaSize);
				// Allocate space to store the non-contiguous compressed blocks from the workers.
				CompressedBlocks.SetNum(int32(BlockCount));
			}
			return {};
		},
		// Executed in parallel to encode blocks. Blocks are contiguous when there is only one thread.
		[this, &RawHash, &RawData, RawSize, BlockSize, &CompressedBlocks, &CompressedBlockSizes](FBlockContext& BlockContext, const int32 BlockIndex)
		{
			const uint64 RawOffset = BlockIndex * BlockSize;
			const uint64 RawBlockSize = FMath::Min(RawSize - RawOffset, BlockSize);
			const FMemoryView RawBlock = RawData.ViewOrCopyRange(RawOffset, RawBlockSize, BlockContext.RawBlockCopy);

			FUniqueBuffer CompressedBlock;
			FMutableMemoryView CompressedBlockView = BlockContext.CompressedBlocksView;
			if (CompressedBlockView.IsEmpty())
			{
				// Allocate compressed blocks on worker threads when there is more than one thread.
				CompressedBlock = FUniqueBuffer::Alloc(CompressBlockBound(RawBlockSize));
				CompressedBlockView = CompressedBlock;
			}
			else
			{
				// Interleave hashing with encoding when there is only one thread.
				RawHash.Update(RawBlock);
			}

			if (!CompressBlock(CompressedBlockView, RawBlock))
			{
				return;
			}

			uint64 CompressedBlockSize = CompressedBlockView.GetSize();
			if (RawBlockSize <= CompressedBlockSize)
			{
				// Encoding did not compress the block. Copy the raw block.
				CompressedBlockSize = RawBlockSize;
				CompressedBlockView.CopyFrom(RawBlock);
				CompressedBlockView.LeftInline(CompressedBlockSize);
			}

			if (CompressedBlock.IsNull())
			{
				// Blocks are contiguous. Update the view to the start of the next block.
				BlockContext.CompressedBlocksView += CompressedBlockSize;
			}
			else
			{
				// Blocks are non-contiguous. Store the view of this block to gather into a composite buffer.
				CompressedBlocks[BlockIndex] = FSharedBuffer::MakeView(CompressedBlockView, CompressedBlock.MoveToShared());
			}

			// Record the compressed block size. A size of 0 is an error and is handled by CompressBlock.
			check(CompressedBlockSize > 0);
			CompressedBlockSizes[BlockIndex] = uint32(CompressedBlockSize);
		},
		// Executed on the calling thread before it joins in with encoding blocks.
		[&RawHash, &RawData, &BlockContexts]
		{
			if (BlockContexts.Num() > 1)
			{
				// Hash the raw data in parallel with encoding the blocks when there are multiple threads.
				// @todo : on many-core systems like ThreadRippers this is the bottleneck (slower than compression)
				//	use FXxHash64::HashBufferChunked instead.
				TRACE_CPUPROFILER_EVENT_SCOPE(FBlockEncoder.RawHash);
				RawHash.Update(RawData);
			}
		});

	// Accumulate compressed size and check for failed blocks.
	for (const uint64 CompressedBlockSize : CompressedBlockSizes)
	{
		if (CompressedBlockSize == 0)
		{
			return FCompositeBuffer();
		}
		CompressedSize += CompressedBlockSize;
	}

	// Do not return compressed data unless it is smaller than the raw data.
	if (RawSize <= MetaSize + CompressedSize)
	{
		return FCompositeBuffer();
	}

	// Write the header and calculate the CRC-32.
	Algo::ForEach(CompressedBlockSizes, [](uint32& Size) { Size = NETWORK_ORDER32(Size); });
	CompressedData.GetView().Mid(sizeof(FHeader), MetaSize).CopyFrom(MakeMemoryView(CompressedBlockSizes));

	FHeader Header;
	Header.Method = GetMethod();
	Header.Compressor = GetCompressor();
	Header.CompressionLevel = GetCompressionLevel();
	Header.BlockSizeExponent = uint8(FMath::FloorLog2_64(BlockSize));
	Header.BlockCount = IntCastChecked<uint32>(BlockCount);
	Header.TotalRawSize = RawSize;
	Header.TotalCompressedSize = sizeof(FHeader) + MetaSize + CompressedSize;
	Header.RawHash = RawHash.Finalize();
	Header.Write(CompressedData.GetView().Left(sizeof(FHeader) + MetaSize));

	const FMemoryView CompositeView = CompressedData.GetView().Left(Header.TotalCompressedSize);
	return FCompositeBuffer(FSharedBuffer::MakeView(CompositeView, CompressedData.MoveToShared()), MoveTemp(CompressedBlocks));
}

class FBlockDecoder : public FDecoder
{
public:
	uint64 GetHeaderSize(const FHeader& Header) const final
	{
		return sizeof(FHeader) + sizeof(uint32) * uint64(Header.BlockCount);
	}

	bool TryDecompressTo(FDecoderContext& Context, const FDecoderSource& Source, const FHeader& Header, FMemoryView HeaderView, uint64 RawOffset, FMutableMemoryView RawView, ECompressedBufferDecompressFlags Flags) const final;
	FCompositeBuffer DecompressToComposite(FDecoderContext& Context, const FDecoderSource& Source, const FHeader& Header, const FMemoryView HeaderView, uint64 RawOffset, uint64 RawSize) const final;

protected:
	virtual bool DecompressBlock(FMutableMemoryView RawData, FMemoryView CompressedData) const = 0;

#if WITH_EDITORONLY_DATA
private:
	bool TryParallelDecompressTo(FDecoderContext& Context, const FDecoderSource& Source, const FHeader& Header, FMemoryView HeaderView, uint64 RawOffset, FMutableMemoryView RawView, ECompressedBufferDecompressFlags Flags) const;
#endif
};

bool FBlockDecoder::TryDecompressTo(
	FDecoderContext& Context,
	const FDecoderSource& Source,
	const FHeader& Header,
	const FMemoryView HeaderView,
	const uint64 RawOffset,
	const FMutableMemoryView RawView,
	const ECompressedBufferDecompressFlags Flags) const
{
	if (Header.TotalRawSize < RawOffset + RawView.GetSize())
	{
		return false;
	}

	const uint32 FirstBlockIndex = uint32(RawOffset >> Header.BlockSizeExponent);
	const uint32 LastBlockIndex = uint32((RawOffset + RawView.GetSize() - 1) >> Header.BlockSizeExponent);

#if WITH_EDITORONLY_DATA
	if (RawView.GetSize() >= ParallelDecodeMinRawSize && LastBlockIndex - FirstBlockIndex > 1 && Source.SupportsParallelRead())
	{
		return TryParallelDecompressTo(Context, Source, Header, HeaderView, RawOffset, RawView, Flags);
	}
#endif

	TRACE_CPUPROFILER_EVENT_SCOPE(FBlockDecoder::TryDecompressTo);

	const uint32* const CompressedBlockSizes = static_cast<const uint32*>(HeaderView.RightChop(sizeof(FHeader)).GetData());
	uint64 CompressedOffset = sizeof(FHeader) + sizeof(uint32) * uint32(Header.BlockCount) +
		Algo::TransformAccumulate(MakeArrayView(CompressedBlockSizes, FirstBlockIndex),
			[](uint32 Size) -> uint64 { return NETWORK_ORDER32(Size); }, uint64(0));

	for (uint32 BlockIndex = FirstBlockIndex; BlockIndex <= LastBlockIndex; ++BlockIndex)
	{
		const auto BlocksToBytes = [&Header](uint64 BlockCount) -> uint64 { return BlockCount << Header.BlockSizeExponent; };
		const bool bFirstBlock = (BlockIndex == FirstBlockIndex);
		const uint64 BlockSize = BlocksToBytes(1);
		const uint64 RawBlockOffset = bFirstBlock ? RawOffset & (BlockSize - 1) : 0;
		const uint64 RawBlockSize = FMath::Min(BlockSize, Header.TotalRawSize - BlocksToBytes(BlockIndex));
		const FMutableMemoryView RawTarget = RawView.Mid(bFirstBlock ? 0 : BlocksToBytes(BlockIndex) - RawOffset, RawBlockSize - RawBlockOffset);
		const uint32 CompressedBlockSize = NETWORK_ORDER32(CompressedBlockSizes[BlockIndex]);
		const bool bIsCompressed = CompressedBlockSize < RawBlockSize;

		if (bIsCompressed)
		{
			if (Context.RawBlockIndex == BlockIndex)
			{
				RawTarget.CopyFrom(Context.RawBlock.GetView().Mid(RawBlockOffset, RawTarget.GetSize()));
			}
			else
			{
				FMutableMemoryView RawBlock;
				if (RawTarget.GetSize() == RawBlockSize && !EnumHasAnyFlags(Flags, ECompressedBufferDecompressFlags::IntermediateBuffer))
				{
					RawBlock = RawTarget;
				}
				else
				{
					if (Context.RawBlock.GetSize() < RawBlockSize)
					{
						Context.RawBlock = FUniqueBuffer::Alloc(BlockSize);
					}
					RawBlock = Context.RawBlock.GetView().Left(RawBlockSize);
					Context.RawBlockIndex = BlockIndex;
				}

				const FMemoryView CompressedBlock = Source.ReadOrView(CompressedOffset, CompressedBlockSize, Context);
				if (CompressedBlock.IsEmpty() || !DecompressBlock(RawBlock, CompressedBlock))
				{
					return false;
				}

				if (RawTarget.GetSize() != RawBlockSize || EnumHasAnyFlags(Flags, ECompressedBufferDecompressFlags::IntermediateBuffer))
				{
					RawTarget.CopyFrom(RawBlock.Mid(RawBlockOffset, RawTarget.GetSize()));
				}
			}
		}
		else
		{
			Source.Read(CompressedOffset + RawBlockOffset, RawTarget);
		}

		CompressedOffset += CompressedBlockSize;
	}

	return true;
}

#if WITH_EDITORONLY_DATA
FORCENOINLINE bool FBlockDecoder::TryParallelDecompressTo(
	FDecoderContext& Context,
	const FDecoderSource& Source,
	const FHeader& Header,
	const FMemoryView HeaderView,
	const uint64 RawOffset,
	const FMutableMemoryView RawView,
	const ECompressedBufferDecompressFlags Flags) const
{
	if (Header.TotalRawSize < RawOffset + RawView.GetSize())
	{
		return false;
	}

	const uint32 FirstBlockIndex = uint32(RawOffset >> Header.BlockSizeExponent);
	const uint32 LastBlockIndex = uint32((RawOffset + RawView.GetSize() - 1) >> Header.BlockSizeExponent);

	TRACE_CPUPROFILER_EVENT_SCOPE(FBlockDecoder::TryParallelDecompressTo);

	const uint32* const CompressedBlockSizes = static_cast<const uint32*>(HeaderView.RightChop(sizeof(FHeader)).GetData());
	TArray<uint64, TInlineAllocator<64>> CompressedOffsets;
	CompressedOffsets.Reserve(LastBlockIndex - FirstBlockIndex + 1);
	CompressedOffsets.Add(sizeof(FHeader) + sizeof(uint32) * uint32(Header.BlockCount) +
		Algo::TransformAccumulate(MakeArrayView(CompressedBlockSizes, FirstBlockIndex),
			[](uint32 Size) -> uint64 { return NETWORK_ORDER32(Size); }, uint64(0)));
	for (uint32 BlockIndex = FirstBlockIndex; BlockIndex < LastBlockIndex; ++BlockIndex)
	{
		CompressedOffsets.Add(CompressedOffsets.Last() + NETWORK_ORDER32(CompressedBlockSizes[BlockIndex]));
	}

	const auto TryDecompressBlock = [this, &Source, &Header, RawOffset, RawView, Flags, CompressedBlockSizes, CompressedOffsets = CompressedOffsets.GetData(), FirstBlockIndex](FDecoderContext& Context, const uint32 BlockIndex)
	{
		const auto BlocksToBytes = [&Header](uint64 BlockCount) -> uint64 { return BlockCount << Header.BlockSizeExponent; };
		const bool bFirstBlock = (BlockIndex == FirstBlockIndex);
		const uint64 BlockSize = BlocksToBytes(1);
		const uint64 RawBlockOffset = bFirstBlock ? RawOffset & (BlockSize - 1) : 0;
		const uint64 RawBlockSize = FMath::Min(BlockSize, Header.TotalRawSize - BlocksToBytes(BlockIndex));
		const FMutableMemoryView RawTarget = RawView.Mid(bFirstBlock ? 0 : BlocksToBytes(BlockIndex) - RawOffset, RawBlockSize - RawBlockOffset);
		const uint64 CompressedOffset = CompressedOffsets[BlockIndex - FirstBlockIndex];
		const uint32 CompressedBlockSize = NETWORK_ORDER32(CompressedBlockSizes[BlockIndex]);
		const bool bIsCompressed = CompressedBlockSize < RawBlockSize;

		if (bIsCompressed)
		{
			if (Context.RawBlockIndex == BlockIndex)
			{
				RawTarget.CopyFrom(Context.RawBlock.GetView().Mid(RawBlockOffset, RawTarget.GetSize()));
			}
			else
			{
				FMutableMemoryView RawBlock;
				if (RawTarget.GetSize() == RawBlockSize && !EnumHasAnyFlags(Flags, ECompressedBufferDecompressFlags::IntermediateBuffer))
				{
					RawBlock = RawTarget;
				}
				else
				{
					if (Context.RawBlock.GetSize() < RawBlockSize)
					{
						Context.RawBlock = FUniqueBuffer::Alloc(BlockSize);
					}
					RawBlock = Context.RawBlock.GetView().Left(RawBlockSize);
					Context.RawBlockIndex = BlockIndex;
				}

				const FMemoryView CompressedBlock = Source.ReadOrView(CompressedOffset, CompressedBlockSize, Context);
				if (CompressedBlock.IsEmpty() || !DecompressBlock(RawBlock, CompressedBlock))
				{
					return false;
				}

				if (RawTarget.GetSize() != RawBlockSize || EnumHasAnyFlags(Flags, ECompressedBufferDecompressFlags::IntermediateBuffer))
				{
					RawTarget.CopyFrom(RawBlock.Mid(RawBlockOffset, RawTarget.GetSize()));
				}
			}
		}
		else
		{
			Source.Read(CompressedOffset + RawBlockOffset, RawTarget);
		}

		return true;
	};

	// Track whether any block failed to decompress.
	std::atomic<bool> bSuccess = true;
	// Share decoder contexts among blocks in the same task. These will be unused for most inputs.
	TArray<FDecoderContext, TInlineAllocator<16>> ParallelContexts;
	// Decompress the first and last blocks as pre-work because they are the two that may use Context.RawBlock.
	const int32 ParallelBlockCount = int32(LastBlockIndex - FirstBlockIndex - 1);
	// Require a minimum batch of 1 MiB raw blocks to avoid tasks that are too tiny to be worthwhile.
	const int32 ParallelMinBatchSize = (Header.BlockSizeExponent < 20) ? (1 << (20 - Header.BlockSizeExponent)) : 1;
	// Limit task count because decoding saturates memory bandwidth with very few threads.
	static const int32 MaxTaskCount = FMath::Max(3, FPlatformMisc::NumberOfCoresIncludingHyperthreads() / 4);

	ParallelContexts.SetNum(FMath::Min(MaxTaskCount, ParallelBlockCount));
	ParallelForWithPreWorkWithExistingTaskContext(TEXT("BlockDecoder.TryDecompressTo.PF"),
		MakeArrayView(ParallelContexts), ParallelBlockCount, ParallelMinBatchSize,
		[&TryDecompressBlock, &bSuccess, FirstBlockIndex](FDecoderContext& TaskContext, const int32 ParallelBlockIndex)
		{
			if (!TryDecompressBlock(TaskContext, uint32(ParallelBlockIndex + FirstBlockIndex + 1)))
			{
				bSuccess.store(false, std::memory_order_relaxed);
			}
		},
		[&TryDecompressBlock, &bSuccess, &Context, FirstBlockIndex, LastBlockIndex]
		{
			if (!TryDecompressBlock(Context, FirstBlockIndex) || !TryDecompressBlock(Context, LastBlockIndex))
			{
				bSuccess.store(false, std::memory_order_relaxed);
			}
		},
		EParallelForFlags::Unbalanced);

	return bSuccess.load(std::memory_order_relaxed);
}
#endif // WITH_EDITORONLY_DATA

FCompositeBuffer FBlockDecoder::DecompressToComposite(
	FDecoderContext& Context,
	const FDecoderSource& Source,
	const FHeader& Header,
	const FMemoryView HeaderView,
	const uint64 RawOffset,
	const uint64 RawSize) const
{
	FUniqueBuffer Buffer = FUniqueBuffer::Alloc(RawSize);
	if (TryDecompressTo(Context, Source, Header, HeaderView, RawOffset, Buffer, ECompressedBufferDecompressFlags::None))
	{
		return FCompositeBuffer(Buffer.MoveToShared());
	}
	return FCompositeBuffer();
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

class FOodleEncoder final : public FBlockEncoder
{
public:
	FOodleEncoder(ECompressedBufferCompressor InCompressor, ECompressedBufferCompressionLevel InCompressionLevel)
		: Compressor(InCompressor)
		, CompressionLevel(InCompressionLevel)
	{
	}

protected:
	EMethod GetMethod() const final { return EMethod::Oodle; }
	uint8 GetCompressor() const final { return uint8(Compressor); }
	uint8 GetCompressionLevel() const final { return uint8(CompressionLevel); }

	uint64 CompressBlockBound(uint64 RawSize) const final
	{
		return uint64(FOodleDataCompression::CompressedBufferSizeNeeded(int64(RawSize)));
	}

	bool CompressBlock(FMutableMemoryView& CompressedData, const FMemoryView RawData) const final
	{
		const int64 Size = FOodleDataCompression::Compress(
			CompressedData.GetData(), uint64(CompressedData.GetSize()),
			RawData.GetData(), int64(RawData.GetSize()),
			Compressor, CompressionLevel);
		CompressedData.LeftInline(uint64(Size));
		return Size > 0;
	}

private:
	const ECompressedBufferCompressor Compressor;
	const ECompressedBufferCompressionLevel CompressionLevel;
};

class FOodleDecoder final : public FBlockDecoder
{
protected:
	bool DecompressBlock(const FMutableMemoryView RawData, const FMemoryView CompressedData) const final
	{
		return FOodleDataCompression::Decompress(
			RawData.GetData(), int64(RawData.GetSize()),
			CompressedData.GetData(), int64(CompressedData.GetSize()));
	}
};

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

class FLZ4Encoder final : public FBlockEncoder
{
protected:
	EMethod GetMethod() const final { return EMethod::LZ4; }
	uint8 GetCompressor() const final { return 0; }
	uint8 GetCompressionLevel() const final { return 0; }

	uint64 CompressBlockBound(uint64 RawSize) const final
	{
		if (RawSize <= LZ4_MAX_INPUT_SIZE)
		{
			return uint64(LZ4_compressBound(int(RawSize)));
		}
		return 0;
	}

	bool CompressBlock(FMutableMemoryView& CompressedData, const FMemoryView RawData) const final
	{
		if (RawData.GetSize() <= LZ4_MAX_INPUT_SIZE)
		{
			const int Size = LZ4_compress_default(
				static_cast<const char*>(RawData.GetData()), static_cast<char*>(CompressedData.GetData()),
				int(RawData.GetSize()), int(FMath::Min<uint64>(CompressedData.GetSize(), MAX_int32)));
			CompressedData.LeftInline(uint64(Size));
			return Size > 0;
		}
		return false;
	}
};

class FLZ4Decoder final : public FBlockDecoder
{
protected:
	bool DecompressBlock(const FMutableMemoryView RawData, const FMemoryView CompressedData) const final
	{
		if (CompressedData.GetSize() <= MAX_int32)
		{
			const int Size = LZ4_decompress_safe(
				static_cast<const char*>(CompressedData.GetData()),
				static_cast<char*>(RawData.GetData()),
				int(CompressedData.GetSize()),
				int(FMath::Min<uint64>(RawData.GetSize(), LZ4_MAX_INPUT_SIZE)));
			return uint64(Size) == RawData.GetSize();
		}
		return false;
	}
};

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

static const FDecoder* GetDecoder(const EMethod Method)
{
	static FNoneDecoder None;
	static FOodleDecoder Oodle;
	static FLZ4Decoder LZ4;
	switch (Method)
	{
	default:
		return nullptr;
	case EMethod::None:
		return &None;
	case EMethod::Oodle:
		return &Oodle;
	case EMethod::LZ4:
		return &LZ4;
	}
}

template <typename BufferType>
inline FCompositeBuffer ValidBufferOrEmpty(BufferType&& CompressedData)
{
	return FHeader::IsValid(CompressedData) ? FCompositeBuffer(Forward<BufferType>(CompressedData)) : FCompositeBuffer();
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

bool FHeader::IsValid(const FCompositeBuffer& CompressedData)
{
	const uint64 CompressedDataSize = CompressedData.GetSize();
	if (sizeof(FHeader) <= CompressedDataSize)
	{
		const FHeader Header = Read(CompressedData);
		if (Header.Magic == FHeader::ExpectedMagic && Header.TotalCompressedSize <= CompressedDataSize)
		{
			if (const FDecoder* const Decoder = GetDecoder(Header.Method))
			{
				FUniqueBuffer HeaderCopy;
				const FMemoryView HeaderView = CompressedData.ViewOrCopyRange(0, Decoder->GetHeaderSize(Header), HeaderCopy);
				if (Header.Crc32 == FHeader::CalculateCrc32(HeaderView))
				{
					return true;
				}
			}
		}
	}
	return false;
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

static bool TryReadHeader(FDecoderContext& Context, FArchive& Ar, FHeader& OutHeader, FMemoryView& OutHeaderView)
{
	if (Context.HeaderOffset != MAX_uint64)
	{
		OutHeaderView = Context.Header.GetView().Left(Context.HeaderSize);
		MakeMemoryView(&OutHeader, &OutHeader + 1).CopyFrom(OutHeaderView.Left(sizeof(FHeader)));
		OutHeader.ByteSwap();
		return true;
	}

	check(Ar.IsLoading());
	const int64 Offset = Ar.Tell();
	if (Offset == INDEX_NONE)
	{
		return false;
	}

	FHeader& Header = OutHeader;
	Ar.Serialize(&Header, sizeof(FHeader));
	Header.ByteSwap();

	if (const FDecoder* const Decoder = GetDecoder(Header.Method); Decoder && Header.Magic == FHeader::ExpectedMagic)
	{
		const uint64 HeaderSize = Decoder->GetHeaderSize(Header);
		if (Context.Header.GetSize() < HeaderSize)
		{
			Context.Header = FUniqueBuffer::Alloc(FMath::Max(FMath::RoundUpToPowerOfTwo64(HeaderSize), DefaultHeaderSize));
		}

		const FMutableMemoryView HeaderView = Context.Header.GetView().Left(HeaderSize);
		const FMutableMemoryView HeaderTail = HeaderView.CopyFrom(MakeMemoryView(&Header, &Header + 1));
		Ar.Serialize(HeaderTail.GetData(), int64(HeaderTail.GetSize()));

		FHeader* const HeaderCopy = static_cast<FHeader*>(HeaderView.GetData());
		HeaderCopy->ByteSwap();
		if (Header.Crc32 == FHeader::CalculateCrc32(HeaderView))
		{
			Context.HeaderOffset = uint64(Offset);
			Context.HeaderSize = HeaderSize;
			Context.HeaderCrc32 = Header.Crc32;
			Context.RawBlockIndex = MAX_uint32;
			OutHeaderView = HeaderView;
			return true;
		}
	}

	return false;
}

static bool TryReadHeader(FDecoderContext& Context, const FCompositeBuffer& Buffer, FHeader& OutHeader, FMemoryView& OutHeaderView)
{
	if (Context.HeaderOffset != MAX_uint64)
	{
		OutHeaderView = Buffer.ViewOrCopyRange(Context.HeaderOffset, Context.HeaderSize, Context.Header);
		MakeMemoryView(&OutHeader, &OutHeader + 1).CopyFrom(OutHeaderView.Left(sizeof(FHeader)));
		OutHeader.ByteSwap();
		return true;
	}

	if (Buffer.GetSize() < sizeof(FHeader))
	{
		return false;
	}

	FHeader& Header = OutHeader;
	Buffer.CopyTo(MakeMemoryView(&Header, &Header + 1));
	Header.ByteSwap();

	if (const FDecoder* const Decoder = GetDecoder(Header.Method); Decoder && Header.Magic == FHeader::ExpectedMagic)
	{
		const uint64 HeaderSize = Decoder->GetHeaderSize(Header);
		const FMemoryView HeaderView = Buffer.ViewOrCopyRange(0, HeaderSize, Context.Header,
			[](uint64 Size) { return FUniqueBuffer::Alloc(FMath::Max(FMath::RoundUpToPowerOfTwo64(Size), DefaultHeaderSize)); });
		if (Header.Crc32 == FHeader::CalculateCrc32(HeaderView))
		{
			Context.HeaderOffset = 0;
			Context.HeaderSize = HeaderSize;
			if (Context.HeaderCrc32 != Header.Crc32 || Header.RawHash.IsZero())
			{
				Context.HeaderCrc32 = Header.Crc32;
				Context.RawBlockIndex = MAX_uint32;
			}
			OutHeaderView = HeaderView;
			return true;
		}
	}

	return false;
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

class FArchiveDecoderSource final : public FDecoderSource
{
public:
	explicit FArchiveDecoderSource(FArchive& InArchive, const uint64 InBaseOffset)
		: Archive(InArchive)
		, BaseOffset(InBaseOffset)
	{
	}

	bool SupportsParallelRead() const final
	{
		return false;
	}

	bool Read(uint64 Offset, FMutableMemoryView Data) const final
	{
		Archive.Seek(int64(BaseOffset + Offset));
		Archive.Serialize(Data.GetData(), int64(Data.GetSize()));
		return !Archive.IsError();
	}

	FMemoryView ReadOrView(const uint64 Offset, const uint64 Size, FDecoderContext& Context) const final
	{
		if (Context.CompressedBlock.GetSize() < Size)
		{
			Context.CompressedBlock = FUniqueBuffer::Alloc(FMath::Max(FMath::RoundUpToPowerOfTwo64(Size), DefaultBlockSize));
		}
		const FMutableMemoryView View = Context.CompressedBlock.GetView().Left(Size);
		return Read(Offset, View) ? View : FMemoryView();
	}

	FCompositeBuffer ReadToComposite(const uint64 Offset, const uint64 Size) const final
	{
		FUniqueBuffer Buffer = FUniqueBuffer::Alloc(Size);
		if (Read(Offset, Buffer))
		{
			return FCompositeBuffer(Buffer.MoveToShared());
		}
		return FCompositeBuffer();
	}

private:
	FArchive& Archive;
	const uint64 BaseOffset;
};

class FBufferDecoderSource final : public FDecoderSource
{
public:
	explicit FBufferDecoderSource(const FCompositeBuffer& InBuffer)
		: Buffer(InBuffer)
	{
	}

	bool SupportsParallelRead() const final
	{
		return true;
	}

	bool Read(const uint64 Offset, const FMutableMemoryView Data) const final
	{
		if (Offset + Data.GetSize() <= Buffer.GetSize())
		{
			Buffer.CopyTo(Data, Offset);
			return true;
		}
		return false;
	}

	FMemoryView ReadOrView(const uint64 Offset, const uint64 Size, FDecoderContext& Context) const final
	{
		return Buffer.ViewOrCopyRange(Offset, Size, Context.CompressedBlock, [](uint64 BufferSize) -> FUniqueBuffer
		{
			return FUniqueBuffer::Alloc(FMath::Max(FMath::RoundUpToPowerOfTwo64(BufferSize), DefaultBlockSize));
		});
	}

	FCompositeBuffer ReadToComposite(const uint64 Offset, const uint64 Size) const final
	{
		return Buffer.Mid(Offset, Size).MakeOwned();
	}

private:
	const FCompositeBuffer& Buffer;
};

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

} // UE::CompressedBuffer::Private

const FCompressedBuffer FCompressedBuffer::Null;

FCompressedBuffer FCompressedBuffer::Compress(const FCompositeBuffer& RawData)
{
	return Compress(RawData, ECompressedBufferCompressor::Mermaid, ECompressedBufferCompressionLevel::VeryFast);
}

FCompressedBuffer FCompressedBuffer::Compress(const FSharedBuffer& RawData)
{
	return Compress(FCompositeBuffer(RawData));
}

FCompressedBuffer FCompressedBuffer::Compress(
	const FCompositeBuffer& RawData,
	const ECompressedBufferCompressor Compressor,
	const ECompressedBufferCompressionLevel CompressionLevel,
	uint64 BlockSize)
{
	using namespace UE::CompressedBuffer::Private;

	if (BlockSize == 0)
	{
		BlockSize = DefaultBlockSize;
	}

	FCompressedBuffer Local;
	if (CompressionLevel != ECompressedBufferCompressionLevel::None)
	{
		Local.CompressedData = FOodleEncoder(Compressor, CompressionLevel).Compress(RawData, BlockSize);
	}
	if (Local.CompressedData.IsNull())
	{
		Local.CompressedData = FNoneEncoder().Compress(RawData, BlockSize);
	}
	return Local;
}

FCompressedBuffer FCompressedBuffer::Compress(
	const FSharedBuffer& RawData,
	const ECompressedBufferCompressor Compressor,
	const ECompressedBufferCompressionLevel CompressionLevel,
	const uint64 BlockSize)
{
	return Compress(FCompositeBuffer(RawData), Compressor, CompressionLevel, BlockSize);
}

FCompressedBuffer FCompressedBuffer::FromCompressed(const FCompositeBuffer& InCompressedData)
{
	FCompressedBuffer Local;
	Local.CompressedData = UE::CompressedBuffer::Private::ValidBufferOrEmpty(InCompressedData);
	return Local;
}

FCompressedBuffer FCompressedBuffer::FromCompressed(FCompositeBuffer&& InCompressedData)
{
	FCompressedBuffer Local;
	Local.CompressedData = UE::CompressedBuffer::Private::ValidBufferOrEmpty(MoveTemp(InCompressedData));
	return Local;
}

FCompressedBuffer FCompressedBuffer::FromCompressed(const FSharedBuffer& InCompressedData)
{
	FCompressedBuffer Local;
	Local.CompressedData = UE::CompressedBuffer::Private::ValidBufferOrEmpty(InCompressedData);
	return Local;
}

FCompressedBuffer FCompressedBuffer::FromCompressed(FSharedBuffer&& InCompressedData)
{
	FCompressedBuffer Local;
	Local.CompressedData = UE::CompressedBuffer::Private::ValidBufferOrEmpty(MoveTemp(InCompressedData));
	return Local;
}

FCompressedBuffer FCompressedBuffer::Load(FArchive& Ar)
{
	using namespace UE::CompressedBuffer::Private;
	check(Ar.IsLoading());

	FHeader Header;
	Ar.Serialize(&Header, sizeof(FHeader));
	Header.ByteSwap();

	FCompressedBuffer Local;
	constexpr uint64 MaxCompressedSize = uint64(1) << 48;
	if (Header.Magic == Header.ExpectedMagic && Header.TotalCompressedSize >= sizeof(FHeader) && Header.TotalCompressedSize <= MaxCompressedSize)
	{
		FUniqueBuffer MutableBuffer = FUniqueBuffer::Alloc(Header.TotalCompressedSize);
		Header.ByteSwap();
		const FMutableMemoryView MutableView = MutableBuffer.GetView().CopyFrom(MakeMemoryView(&Header, &Header + 1));
		Ar.Serialize(MutableView.GetData(), int64(MutableView.GetSize()));
		if (!Ar.IsError())
		{
			Local.CompressedData = ValidBufferOrEmpty(MutableBuffer.MoveToShared());
		}
	}
	if (Local.IsNull())
	{
		Ar.SetError();
	}
	return Local;
}

void FCompressedBuffer::Save(FArchive& Ar) const
{
	check(Ar.IsSaving());
	checkf(!CompressedData.IsNull(), TEXT("Serializing a null compressed buffer is not supported."));
	for (const FSharedBuffer& Segment : CompressedData.GetSegments())
	{
		Ar.Serialize(const_cast<void*>(Segment.GetData()), int64(Segment.GetSize()));
	}
}

uint64 FCompressedBuffer::GetCompressedSize() const
{
	return CompressedData.GetSize();
}

uint64 FCompressedBuffer::GetRawSize() const
{
	return CompressedData ? UE::CompressedBuffer::Private::FHeader::Read(CompressedData).TotalRawSize : 0;
}

FIoHash FCompressedBuffer::GetRawHash() const
{
	return CompressedData ? UE::CompressedBuffer::Private::FHeader::Read(CompressedData).RawHash : FIoHash();
}

bool FCompressedBuffer::TryGetCompressParameters(
	ECompressedBufferCompressor& OutCompressor,
	ECompressedBufferCompressionLevel& OutCompressionLevel,
	uint64& OutBlockSize) const
{
	using namespace UE::CompressedBuffer::Private;
	if (CompressedData)
	{
		return FHeader::Read(CompressedData).TryGetCompressParameters(OutCompressor, OutCompressionLevel, OutBlockSize);
	}
	return false;
}

bool FCompressedBuffer::TryDecompressTo(const FMutableMemoryView RawView, ECompressedBufferDecompressFlags Flags) const
{
	if (CompressedData && RawView.GetSize() == GetRawSize())
	{
		return FCompressedBufferReader(*this).TryDecompressTo(RawView, 0, Flags);
	}
	return false;
}

FSharedBuffer FCompressedBuffer::Decompress() const
{
	if (CompressedData)
	{
		return FCompressedBufferReader(*this).Decompress();
	}
	return FSharedBuffer();
}

FCompositeBuffer FCompressedBuffer::DecompressToComposite() const
{
	if (CompressedData)
	{
		return FCompressedBufferReader(*this).DecompressToComposite();
	}
	return FCompositeBuffer();
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FArchive& operator<<(FArchive& Ar, FCompressedBuffer& Buffer)
{
	if (Ar.IsLoading())
	{
		Buffer = FCompressedBuffer::Load(Ar);
	}
	else
	{
		Buffer.Save(Ar);
	}
	return Ar;
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FCompressedBufferReader::FCompressedBufferReader(FArchive& Archive)
{
	SetSource(Archive);
}

FCompressedBufferReader::FCompressedBufferReader(const FCompressedBuffer& Buffer)
{
	SetSource(Buffer);
}

void FCompressedBufferReader::ResetBuffers()
{
	using namespace UE::CompressedBuffer::Private;
	if (SourceArchive && Context.HeaderOffset != MAX_uint64)
	{
		SourceArchive->Seek(int64(Context.HeaderOffset));
	}
	Context = FDecoderContext();
}

void FCompressedBufferReader::ResetSource()
{
	Context.HeaderOffset = MAX_uint64;
	SourceArchive = nullptr;
	SourceBuffer = nullptr;
}

void FCompressedBufferReader::SetSource(FArchive& Archive)
{
	if (SourceArchive == &Archive)
	{
		return;
	}
	Context.HeaderOffset = MAX_uint64;
	SourceArchive = &Archive;
	SourceBuffer = nullptr;
}

void FCompressedBufferReader::SetSource(const FCompressedBuffer& Buffer)
{
	if (SourceBuffer == &Buffer)
	{
		return;
	}
	Context.HeaderOffset = MAX_uint64;
	SourceArchive = nullptr;
	SourceBuffer = &Buffer;
}

uint64 FCompressedBufferReader::GetCompressedSize()
{
	using namespace UE::CompressedBuffer::Private;
	FHeader Header;
	FMemoryView HeaderView;
	if (TryReadHeader(Header, HeaderView))
	{
		return Header.TotalCompressedSize;
	}
	return 0;
}

uint64 FCompressedBufferReader::GetRawSize()
{
	using namespace UE::CompressedBuffer::Private;
	FHeader Header;
	FMemoryView HeaderView;
	if (TryReadHeader(Header, HeaderView))
	{
		return Header.TotalRawSize;
	}
	return 0;
}

FIoHash FCompressedBufferReader::GetRawHash()
{
	using namespace UE::CompressedBuffer::Private;
	FHeader Header;
	FMemoryView HeaderView;
	if (TryReadHeader(Header, HeaderView))
	{
		return Header.RawHash;
	}
	return FIoHash();
}

bool FCompressedBufferReader::TryGetCompressParameters(
	ECompressedBufferCompressor& OutCompressor,
	ECompressedBufferCompressionLevel& OutCompressionLevel,
	uint64& OutBlockSize)
{
	using namespace UE::CompressedBuffer::Private;
	FHeader Header;
	FMemoryView HeaderView;
	if (TryReadHeader(Header, HeaderView))
	{
		return Header.TryGetCompressParameters(OutCompressor, OutCompressionLevel, OutBlockSize);
	}
	return false;
}

bool FCompressedBufferReader::TryDecompressTo(const FMutableMemoryView RawView, const uint64 RawOffset, ECompressedBufferDecompressFlags Flags)
{
	using namespace UE::CompressedBuffer::Private;
	FHeader Header;
	FMemoryView HeaderView;
	if (TryReadHeader(Header, HeaderView))
	{
		const uint64 TotalRawSize = Header.TotalRawSize;
		if (RawOffset <= TotalRawSize && RawView.GetSize() <= TotalRawSize - RawOffset)
		{
			if (const FDecoder* const Decoder = GetDecoder(Header.Method))
			{
				if (Decoder->TryDecompressTo(Context,
					SourceArchive
						? ImplicitConv<const FDecoderSource&>(FArchiveDecoderSource(*SourceArchive, Context.HeaderOffset))
						: ImplicitConv<const FDecoderSource&>(FBufferDecoderSource(SourceBuffer->GetCompressed())),
					Header, HeaderView, RawOffset, RawView, Flags))
				{
					return true;
				}
			}
		}
	}
	return false;
}

FSharedBuffer FCompressedBufferReader::Decompress(const uint64 RawOffset, const uint64 RawSize)
{
	using namespace UE::CompressedBuffer::Private;
	FHeader Header;
	FMemoryView HeaderView;
	if (TryReadHeader(Header, HeaderView))
	{
		const uint64 TotalRawSize = Header.TotalRawSize;
		const uint64 RawSizeToCopy = RawSize == MAX_uint64 ? TotalRawSize - RawOffset : RawSize;
		if (RawOffset <= TotalRawSize && RawSizeToCopy <= TotalRawSize - RawOffset)
		{
			if (const FDecoder* const Decoder = GetDecoder(Header.Method))
			{
				FUniqueBuffer RawData = FUniqueBuffer::Alloc(RawSizeToCopy);
				if (Decoder->TryDecompressTo(Context,
					SourceArchive
						? ImplicitConv<const FDecoderSource&>(FArchiveDecoderSource(*SourceArchive, Context.HeaderOffset))
						: ImplicitConv<const FDecoderSource&>(FBufferDecoderSource(SourceBuffer->GetCompressed())),
					Header, HeaderView, RawOffset, RawData, ECompressedBufferDecompressFlags::None))
				{
					return RawData.MoveToShared();
				}
			}
		}
	}
	return FSharedBuffer();
}

FCompositeBuffer FCompressedBufferReader::DecompressToComposite(const uint64 RawOffset, const uint64 RawSize)
{
	using namespace UE::CompressedBuffer::Private;
	FHeader Header;
	FMemoryView HeaderView;
	if (TryReadHeader(Header, HeaderView))
	{
		const uint64 TotalRawSize = Header.TotalRawSize;
		const uint64 RawSizeToCopy = RawSize == MAX_uint64 ? TotalRawSize - RawOffset : RawSize;
		if (RawOffset <= TotalRawSize && RawSizeToCopy <= TotalRawSize - RawOffset)
		{
			if (const FDecoder* const Decoder = GetDecoder(Header.Method))
			{
				return Decoder->DecompressToComposite(Context,
					SourceArchive
						? ImplicitConv<const FDecoderSource&>(FArchiveDecoderSource(*SourceArchive, Context.HeaderOffset))
						: ImplicitConv<const FDecoderSource&>(FBufferDecoderSource(SourceBuffer->GetCompressed())),
					Header, HeaderView, RawOffset, RawSizeToCopy);
			}
		}
	}
	return FCompositeBuffer();
}

bool FCompressedBufferReader::TryReadHeader(UE::CompressedBuffer::Private::FHeader& OutHeader, FMemoryView& OutHeaderView)
{
	using namespace UE::CompressedBuffer;
	if (FArchive* const Archive = SourceArchive)
	{
		return Private::TryReadHeader(Context, *Archive, OutHeader, OutHeaderView);
	}
	if (const FCompressedBuffer* const Buffer = SourceBuffer)
	{
		return Private::TryReadHeader(Context, Buffer->GetCompressed(), OutHeader, OutHeaderView);
	}
	return false;
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
