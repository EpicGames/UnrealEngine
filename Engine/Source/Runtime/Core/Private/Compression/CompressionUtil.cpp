// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compression/CompressionUtil.h"

#include "Containers/UnrealString.h"
#include "CoreGlobals.h"
#include "CoreTypes.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Serialization/Archive.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

namespace FCompressionUtil
{


	// Serialize common compressor names in a byte
	//	to avoid going through String serialization and String-to-Name
	void CORE_API SerializeCompressorName(FArchive & Archive,FName & Compressor)
	{
		if ( Archive.IsLoading() )
		{
			uint8 CompressorNum;
			Archive << CompressorNum;

			switch(CompressorNum)
			{
			case 0:
			{
				// can't rely on archive serializing FName, so use String
				FString LoadedString;
				Archive << LoadedString;
				Compressor = FName(*LoadedString);
				break;
			}
			case 1:
				Compressor = NAME_None;
				break;
			case 2:
				Compressor = NAME_Oodle;
				break;
			case 3:
				Compressor = NAME_Zlib;
				break;
			case 4:
				Compressor = NAME_Gzip;
				break;
			case 5:
				Compressor = NAME_LZ4;
				break;
			default:		
				UE_LOGF(LogSerialization, Error, "SerializeCompressorName Unknown index:%d",CompressorNum );
				
				Compressor = NAME_None;
				break;
			}
		}
		else
		{
			uint8 CompressorNum = 0;

			if ( Compressor == NAME_None )
			{
				CompressorNum = 1;
			}
			else if ( Compressor == NAME_Oodle )
			{
				CompressorNum = 2;
			}
			else if ( Compressor == NAME_Zlib )
			{
				CompressorNum = 3;
			}
			else if ( Compressor == NAME_Gzip )
			{
				CompressorNum = 4;
			}
			else if ( Compressor == NAME_LZ4 )
			{
				CompressorNum = 5;
			}
			// else CompressorNum = 0 is right

			
			Archive << CompressorNum;

			if ( CompressorNum == 0 )
			{
				// can't rely on archive serializing FName, so use String
				FString SavedString(Compressor.ToString());
				Archive << SavedString;
			}			
		}
	}

	TArray<FString> CORE_API HexDumpLines(const uint8* Bytes, int64 BytesNum, int64 OffsetStart, int64 OffsetEnd, int32 BytesPerLine)
	{
 		TArray<FString> Results;

		for (int64 Idx = OffsetStart; Idx < OffsetEnd;)
		{
			FString HexString;
			HexString.Reserve(128);
			int64 LineOffset = OffsetStart;
			for (int64 Idx2 = 0; Idx2 < BytesPerLine && Idx < OffsetEnd; ++Idx, ++Idx2, ++OffsetStart)
			{
				if (0 <= Idx && Idx < BytesNum)
				{
					HexString += FString::Printf(TEXT("%02X "), Bytes[Idx]);
				}
				else
				{
					HexString += TEXT("-- ");
				}
				if ((Idx2 & 7) == 7)
				{
					HexString += TEXT(" ");
				}
			}
			if (LineOffset >= 0)
			{
				Results.Add(FString::Printf(TEXT("%016" INT64_X_FMT ": %s"), LineOffset, *HexString));
			}
			else
			{
				Results.Add(FString::Printf(TEXT(" -%014" INT64_X_FMT ": %s"), -LineOffset, *HexString));
			}
		}
		return Results;
	}

	void CORE_API LogHexDump(const uint8* Bytes, int64 BytesNum, int64 OffsetStart, int64 OffsetEnd, int32 BytesPerLine)
	{
		if (UE_LOG_ACTIVE(LogCore, Display))
		{
			TArray<FString> Results = HexDumpLines(Bytes, BytesNum, OffsetStart, OffsetEnd, BytesPerLine);
			for (const FString& Result : Results)
			{
				UE_LOGF(LogCore, Display, "%ls", *Result);
			}
		}
	}
};
