/*
    LZ4 HC - High Compression Mode of LZ4
    Copyright (C) 2011-2020, Yann Collet.

    BSD 2-Clause License (http://www.opensource.org/licenses/bsd-license.php)

    Redistribution and use in source and binary forms, with or without
    modification, are permitted provided that the following conditions are
    met:

    * Redistributions of source code must retain the above copyright
    notice, this list of conditions and the following disclaimer.
    * Redistributions in binary form must reproduce the above
    copyright notice, this list of conditions and the following disclaimer
    in the documentation and/or other materials provided with the
    distribution.

    THIS SOFTWARE IS PROVIDED BY THE COPYRIGHT HOLDERS AND CONTRIBUTORS
    "AS IS" AND ANY EXPRESS OR IMPLIED WARRANTIES, INCLUDING, BUT NOT
    LIMITED TO, THE IMPLIED WARRANTIES OF MERCHANTABILITY AND FITNESS FOR
    A PARTICULAR PURPOSE ARE DISCLAIMED. IN NO EVENT SHALL THE COPYRIGHT
    OWNER OR CONTRIBUTORS BE LIABLE FOR ANY DIRECT, INDIRECT, INCIDENTAL,
    SPECIAL, EXEMPLARY, OR CONSEQUENTIAL DAMAGES (INCLUDING, BUT NOT
    LIMITED TO, PROCUREMENT OF SUBSTITUTE GOODS OR SERVICES; LOSS OF USE,
    DATA, OR PROFITS; OR BUSINESS INTERRUPTION) HOWEVER CAUSED AND ON ANY
    THEORY OF LIABILITY, WHETHER IN CONTRACT, STRICT LIABILITY, OR TORT
    (INCLUDING NEGLIGENCE OR OTHERWISE) ARISING IN ANY WAY OUT OF THE USE
    OF THIS SOFTWARE, EVEN IF ADVISED OF THE POSSIBILITY OF SUCH DAMAGE.

    You can contact the author at :
       - LZ4 source repository : https://github.com/lz4/lz4
       - LZ4 public forum : https://groups.google.com/forum/#!forum/lz4c
*/
/* note : lz4hc is not an independent module, it requires lz4.h/lz4.c for proper compilation */


/* *************************************
*  Tuning Parameter
***************************************/

/*! HEAPMODE :
 *  Select how default compression function will allocate workplace memory,
 *  in stack (0:fastest), or in heap (1:requires malloc()).
 *  Since workplace is rather large, heap mode is recommended.
**/
#ifndef LZ4HC_HEAPMODE
#  define LZ4HC_HEAPMODE 1
#endif


/*===    Dependency    ===*/
#define LZ4_HC_STATIC_LINKING_ONLY
#include "Compression/lz4hc.h"


/*===   Common definitions   ===*/
#if defined(__GNUC__)
#  pragma GCC diagnostic ignored "-Wunused-function"
#endif
#if defined (__clang__)
#  pragma clang diagnostic ignored "-Wunused-function"
#endif

#define LZ4_COMMONDEFS_ONLY
#ifndef LZ4_SRC_INCLUDED
#include "lz4.cpp"   /* LZ4_count, constants, mem */
#endif


/*===   Enums   ===*/
typedef enum { noDictCtx, usingDictCtxHc } dictCtx_directive;


/*===   Constants   ===*/
#define OPTIMAL_ML (int)((ML_MASK-1)+MINMATCH)
#define LZ4_OPT_NUM   (1<<12)


/*===   Macros   ===*/
template<typename T>
static inline T lz4Min(T a, T b)
{
	return (a < b) ? a : b;
}

template<typename T>
static inline T lz4Max(T a, T b)
{
	return (a > b) ? a : b;
}

#define HASH_FUNCTION(i)         (((i) * 2654435761U) >> ((MINMATCH*8)-LZ4HC_HASH_LOG))
#define DELTANEXTMAXD(p)         chainTable[(p) & LZ4HC_MAXD_MASK]    /* flexible, LZ4HC_MAXD dependent */
#define DELTANEXTU16(table, pos) table[(U16)(pos)]   /* faster */
/* Make fields passed to, and updated by LZ4HC_encodeSequence explicit */
#define UPDATABLE(ip, op, anchor) &ip, &op, &anchor

static U32 LZ4HC_hashPtr(const void* ptr) { return HASH_FUNCTION(LZ4_read32(ptr)); }


/**************************************
*  HC Compression
**************************************/
static void LZ4HC_clearTables (LZ4HC_CCtx_internal* hc4)
{
    MEM_INIT(hc4->hashTable, 0, sizeof(hc4->hashTable));
    MEM_INIT(hc4->chainTable, 0xFF, sizeof(hc4->chainTable));
}

static void LZ4HC_init_internal (LZ4HC_CCtx_internal* hc4, const BYTE* start)
{
    size_t const bufferSize = (size_t)(hc4->end - hc4->prefixStart);
    size_t newStartingOffset = bufferSize + hc4->dictLimit;
    assert(newStartingOffset >= bufferSize);  /* check overflow */
    if (newStartingOffset > 1 GB) {
        LZ4HC_clearTables(hc4);
        newStartingOffset = 0;
    }
    newStartingOffset += 64 KB;
    hc4->nextToUpdate = (U32)newStartingOffset;
    hc4->prefixStart = start;
    hc4->end = start;
    hc4->dictStart = start;
    hc4->dictLimit = (U32)newStartingOffset;
    hc4->lowLimit = (U32)newStartingOffset;
}


/* Update chains up to ip (excluded) */
LZ4_FORCE_INLINE void LZ4HC_Insert (LZ4HC_CCtx_internal* hc4, const BYTE* ip)
{
    U16* const chainTable = hc4->chainTable;
    U32* const hashTable  = hc4->hashTable;
    const BYTE* const prefixPtr = hc4->prefixStart;
    U32 const prefixIdx = hc4->dictLimit;
    U32 const target = (U32)(ip - prefixPtr) + prefixIdx;
    U32 idx = hc4->nextToUpdate;
    assert(ip >= prefixPtr);
    assert(target >= prefixIdx);

    while (idx < target) {
        U32 const h = LZ4HC_hashPtr(prefixPtr+idx-prefixIdx);
        size_t delta = idx - hashTable[h];
        if (delta>LZ4_DISTANCE_MAX) delta = LZ4_DISTANCE_MAX;
        DELTANEXTU16(chainTable, idx) = (U16)delta;
        hashTable[h] = idx;
        idx++;
    }

    hc4->nextToUpdate = target;
}

/** LZ4HC_countBack() :
 * @return : negative value, nb of common bytes before ip/match */
LZ4_FORCE_INLINE
int LZ4HC_countBack(const BYTE* const ip, const BYTE* const match,
                    const BYTE* const iMin, const BYTE* const mMin)
{
    int back = 0;
    int const min = (int)lz4Max(iMin - ip, mMin - match);
    assert(min <= 0);
    assert(ip >= iMin); assert((size_t)(ip-iMin) < (1U<<31));
    assert(match >= mMin); assert((size_t)(match - mMin) < (1U<<31));
    while ( (back > min)
         && (ip[back-1] == match[back-1]) )
            back--;
    return back;
}

#if defined(_MSC_VER)
#  define LZ4HC_rotl32(x,r) _rotl(x,r)
#else
#  define LZ4HC_rotl32(x,r) ((x << r) | (x >> (32 - r)))
#endif


static U32 LZ4HC_rotatePattern(size_t const rotate, U32 const pattern)
{
    size_t const bitsToRotate = (rotate & (sizeof(pattern) - 1)) << 3;
    if (bitsToRotate == 0) return pattern;
    return LZ4HC_rotl32(pattern, (int)bitsToRotate);
}

/* LZ4HC_countPattern() :
 * pattern32 must be a sample of repetitive pattern of length 1, 2 or 4 (but not 3!) */
static unsigned
LZ4HC_countPattern(const BYTE* ip, const BYTE* const iEnd, U32 const pattern32)
{
    const BYTE* const iStart = ip;
    reg_t const pattern = (sizeof(pattern)==8) ?
        (reg_t)pattern32 + (((reg_t)pattern32) << (sizeof(pattern)*4)) : pattern32;

    while (likely(ip < iEnd-(sizeof(pattern)-1))) {
        reg_t const diff = LZ4_read_ARCH(ip) ^ pattern;
        if (!diff) { ip+=sizeof(pattern); continue; }
        ip += LZ4_NbCommonBytes(diff);
        return (unsigned)(ip - iStart);
    }

    if (LZ4_isLittleEndian()) {
        reg_t patternByte = pattern;
        while ((ip<iEnd) && (*ip == (BYTE)patternByte)) {
            ip++; patternByte >>= 8;
        }
    } else {  /* big endian */
        U32 bitOffset = (sizeof(pattern)*8) - 8;
        while (ip < iEnd) {
            BYTE const byte = (BYTE)(pattern >> bitOffset);
            if (*ip != byte) break;
            ip ++; bitOffset -= 8;
    }   }

    return (unsigned)(ip - iStart);
}

/* LZ4HC_reverseCountPattern() :
 * pattern must be a sample of repetitive pattern of length 1, 2 or 4 (but not 3!)
 * read using natural platform endianness */
static unsigned
LZ4HC_reverseCountPattern(const BYTE* ip, const BYTE* const iLow, U32 pattern)
{
    const BYTE* const iStart = ip;

    while (likely(ip >= iLow+4)) {
        if (LZ4_read32(ip-4) != pattern) break;
        ip -= 4;
    }
    {   const BYTE* bytePtr = (const BYTE*)(&pattern) + 3; /* works for any endianness */
        while (likely(ip>iLow)) {
            if (ip[-1] != *bytePtr) break;
            ip--; bytePtr--;
    }   }
    return (unsigned)(iStart - ip);
}

/* LZ4HC_protectDictEnd() :
 * Checks if the match is in the last 3 bytes of the dictionary, so reading the
 * 4 byte MINMATCH would overflow.
 * @returns true if the match index is okay.
 */
static int LZ4HC_protectDictEnd(U32 const dictLimit, U32 const matchIndex)
{
    return ((U32)((dictLimit - 1) - matchIndex) >= 3);
}

typedef enum { rep_untested, rep_not, rep_confirmed } repeat_state_e;
typedef enum { favorCompressionRatio=0, favorDecompressionSpeed } HCfavor_e;

LZ4_FORCE_INLINE int
LZ4HC_InsertAndGetWiderMatch (
        LZ4HC_CCtx_internal* const hc4,
        const BYTE* const ip,
        const BYTE* const iLowLimit, const BYTE* const iHighLimit,
        int longest,
        const BYTE** matchpos,
        const BYTE** startpos,
        const int maxNbAttempts,
        const int patternAnalysis, const int chainSwap,
        const dictCtx_directive dict,
        const HCfavor_e favorDecSpeed)
{
    U16* const chainTable = hc4->chainTable;
    U32* const HashTable = hc4->hashTable;
    const LZ4HC_CCtx_internal * const dictCtx = hc4->dictCtx;
    const BYTE* const prefixPtr = hc4->prefixStart;
    const U32 prefixIdx = hc4->dictLimit;
    const U32 ipIndex = (U32)(ip - prefixPtr) + prefixIdx;
    const int withinStartDistance = (hc4->lowLimit + (LZ4_DISTANCE_MAX + 1) > ipIndex);
    const U32 lowestMatchIndex = (withinStartDistance) ? hc4->lowLimit : ipIndex - LZ4_DISTANCE_MAX;
    const BYTE* const dictStart = hc4->dictStart;
    const U32 dictIdx = hc4->lowLimit;
    const BYTE* const dictEnd = dictStart + prefixIdx - dictIdx;
    int const lookBackLength = (int)(ip-iLowLimit);
    int nbAttempts = maxNbAttempts;
    U32 matchChainPos = 0;
    U32 const pattern = LZ4_read32(ip);
    U32 matchIndex;
    repeat_state_e repeat = rep_untested;
    size_t srcPatternLength = 0;

    DEBUGLOG(7, "LZ4HC_InsertAndGetWiderMatch");
    /* First Match */
    LZ4HC_Insert(hc4, ip);
    matchIndex = HashTable[LZ4HC_hashPtr(ip)];
    DEBUGLOG(7, "First match at index %u / %u (lowestMatchIndex)",
                matchIndex, lowestMatchIndex);

    while ((matchIndex>=lowestMatchIndex) && (nbAttempts>0)) {
        int matchLength=0;
        nbAttempts--;
        assert(matchIndex < ipIndex);
        if (favorDecSpeed && (ipIndex - matchIndex < 8)) {
            /* do nothing */
        } else if (matchIndex >= prefixIdx) {   /* within current Prefix */
            const BYTE* const matchPtr = prefixPtr + matchIndex - prefixIdx;
            assert(matchPtr < ip);
            assert(longest >= 1);
            if (LZ4_read16(iLowLimit + longest - 1) == LZ4_read16(matchPtr - lookBackLength + longest - 1)) {
                if (LZ4_read32(matchPtr) == pattern) {
                    int const back = lookBackLength ? LZ4HC_countBack(ip, matchPtr, iLowLimit, prefixPtr) : 0;
                    matchLength = MINMATCH + (int)LZ4_count(ip+MINMATCH, matchPtr+MINMATCH, iHighLimit);
                    matchLength -= back;
                    if (matchLength > longest) {
                        longest = matchLength;
                        *matchpos = matchPtr + back;
                        *startpos = ip + back;
            }   }   }
        } else {   /* lowestMatchIndex <= matchIndex < dictLimit */
            const BYTE* const matchPtr = dictStart + (matchIndex - dictIdx);
            assert(matchIndex >= dictIdx);
            if ( likely(matchIndex <= prefixIdx - 4)
              && (LZ4_read32(matchPtr) == pattern) ) {
                int back = 0;
                const BYTE* vLimit = ip + (prefixIdx - matchIndex);
                if (vLimit > iHighLimit) vLimit = iHighLimit;
                matchLength = (int)LZ4_count(ip+MINMATCH, matchPtr+MINMATCH, vLimit) + MINMATCH;
                if ((ip+matchLength == vLimit) && (vLimit < iHighLimit))
                    matchLength += LZ4_count(ip+matchLength, prefixPtr, iHighLimit);
                back = lookBackLength ? LZ4HC_countBack(ip, matchPtr, iLowLimit, dictStart) : 0;
                matchLength -= back;
                if (matchLength > longest) {
                    longest = matchLength;
                    *matchpos = prefixPtr - prefixIdx + matchIndex + back;   /* virtual pos, relative to ip, to retrieve offset */
                    *startpos = ip + back;
        }   }   }

        if (chainSwap && matchLength==longest) {   /* better match => select a better chain */
            assert(lookBackLength==0);   /* search forward only */
            if (matchIndex + (U32)longest <= ipIndex) {
                int const kTrigger = 4;
                U32 distanceToNextMatch = 1;
                int const end = longest - MINMATCH + 1;
                int step = 1;
                int accel = 1 << kTrigger;
                int pos;
                for (pos = 0; pos < end; pos += step) {
                    U32 const candidateDist = DELTANEXTU16(chainTable, matchIndex + (U32)pos);
                    step = (accel++ >> kTrigger);
                    if (candidateDist > distanceToNextMatch) {
                        distanceToNextMatch = candidateDist;
                        matchChainPos = (U32)pos;
                        accel = 1 << kTrigger;
                }   }
                if (distanceToNextMatch > 1) {
                    if (distanceToNextMatch > matchIndex) break;   /* avoid overflow */
                    matchIndex -= distanceToNextMatch;
                    continue;
        }   }   }

        {   U32 const distNextMatch = DELTANEXTU16(chainTable, matchIndex);
            if (patternAnalysis && distNextMatch==1 && matchChainPos==0) {
                U32 const matchCandidateIdx = matchIndex-1;
                /* may be a repeated pattern */
                if (repeat == rep_untested) {
                    if ( ((pattern & 0xFFFF) == (pattern >> 16))
                      &  ((pattern & 0xFF)   == (pattern >> 24)) ) {
                        repeat = rep_confirmed;
                        srcPatternLength = LZ4HC_countPattern(ip+sizeof(pattern), iHighLimit, pattern) + sizeof(pattern);
                    } else {
                        repeat = rep_not;
                }   }
                if ( (repeat == rep_confirmed) && (matchCandidateIdx >= lowestMatchIndex)
                  && LZ4HC_protectDictEnd(prefixIdx, matchCandidateIdx) ) {
                    const int extDict = matchCandidateIdx < prefixIdx;
                    const BYTE* const matchPtr = (extDict ? dictStart - dictIdx : prefixPtr - prefixIdx) + matchCandidateIdx;
                    if (LZ4_read32(matchPtr) == pattern) {  /* good candidate */
                        const BYTE* const iLimit = extDict ? dictEnd : iHighLimit;
                        size_t forwardPatternLength = LZ4HC_countPattern(matchPtr+sizeof(pattern), iLimit, pattern) + sizeof(pattern);
                        if (extDict && matchPtr + forwardPatternLength == iLimit) {
                            U32 const rotatedPattern = LZ4HC_rotatePattern(forwardPatternLength, pattern);
                            forwardPatternLength += LZ4HC_countPattern(prefixPtr, iHighLimit, rotatedPattern);
                        }
                        {   const BYTE* const lowestMatchPtr = extDict ? dictStart : prefixPtr;
                            size_t backLength = LZ4HC_reverseCountPattern(matchPtr, lowestMatchPtr, pattern);
                            size_t currentSegmentLength;
                            if (!extDict
                              && matchPtr - backLength == prefixPtr
                              && dictIdx < prefixIdx) {
                                U32 const rotatedPattern = LZ4HC_rotatePattern((U32)(-(int)backLength), pattern);
                                backLength += LZ4HC_reverseCountPattern(dictEnd, dictStart, rotatedPattern);
                            }
                            /* Limit backLength not go further than lowestMatchIndex */
                            backLength = matchCandidateIdx - lz4Max(matchCandidateIdx - (U32)backLength, lowestMatchIndex);
                            assert(matchCandidateIdx - backLength >= lowestMatchIndex);
                            currentSegmentLength = backLength + forwardPatternLength;
                            /* Adjust to end of pattern if the source pattern fits, otherwise the beginning of the pattern */
                            if ( (currentSegmentLength >= srcPatternLength)   /* current pattern segment large enough to contain full srcPatternLength */
                              && (forwardPatternLength <= srcPatternLength) ) { /* haven't reached this position yet */
                                U32 const newMatchIndex = matchCandidateIdx + (U32)forwardPatternLength - (U32)srcPatternLength;  /* best position, full pattern, might be followed by more match */
                                if (LZ4HC_protectDictEnd(prefixIdx, newMatchIndex))
                                    matchIndex = newMatchIndex;
                                else {
                                    /* Can only happen if started in the prefix */
                                    assert(newMatchIndex >= prefixIdx - 3 && newMatchIndex < prefixIdx && !extDict);
                                    matchIndex = prefixIdx;
                                }
                            } else {
                                U32 const newMatchIndex = matchCandidateIdx - (U32)backLength;   /* farthest position in current segment, will find a match of length currentSegmentLength + maybe some back */
                                if (!LZ4HC_protectDictEnd(prefixIdx, newMatchIndex)) {
                                    assert(newMatchIndex >= prefixIdx - 3 && newMatchIndex < prefixIdx && !extDict);
                                    matchIndex = prefixIdx;
                                } else {
                                    matchIndex = newMatchIndex;
                                    if (lookBackLength==0) {  /* no back possible */
                                        size_t const maxML = lz4Min(currentSegmentLength, srcPatternLength);
                                        if ((size_t)longest < maxML) {
                                            assert(prefixPtr - prefixIdx + matchIndex != ip);
                                            if ((size_t)(ip - prefixPtr) + prefixIdx - matchIndex > LZ4_DISTANCE_MAX) break;
                                            assert(maxML < 2 GB);
                                            longest = (int)maxML;
                                            *matchpos = prefixPtr - prefixIdx + matchIndex;   /* virtual pos, relative to ip, to retrieve offset */
                                            *startpos = ip;
                                        }
                                        {   U32 const distToNextPattern = DELTANEXTU16(chainTable, matchIndex);
                                            if (distToNextPattern > matchIndex) break;  /* avoid overflow */
                                            matchIndex -= distToNextPattern;
                        }   }   }   }   }
                        continue;
                }   }
        }   }   /* PA optimization */

        /* follow current chain */
        matchIndex -= DELTANEXTU16(chainTable, matchIndex + matchChainPos);

    }  /* while ((matchIndex>=lowestMatchIndex) && (nbAttempts)) */

    if ( dict == usingDictCtxHc
      && nbAttempts > 0
      && ipIndex - lowestMatchIndex < LZ4_DISTANCE_MAX) {
        size_t const dictEndOffset = (size_t)(dictCtx->end - dictCtx->prefixStart) + dictCtx->dictLimit;
        U32 dictMatchIndex = dictCtx->hashTable[LZ4HC_hashPtr(ip)];
        assert(dictEndOffset <= 1 GB);
        matchIndex = dictMatchIndex + lowestMatchIndex - (U32)dictEndOffset;
        while (ipIndex - matchIndex <= LZ4_DISTANCE_MAX && nbAttempts--) {
            const BYTE* const matchPtr = dictCtx->prefixStart - dictCtx->dictLimit + dictMatchIndex;

            if (LZ4_read32(matchPtr) == pattern) {
                int mlt;
                int back = 0;
                const BYTE* vLimit = ip + (dictEndOffset - dictMatchIndex);
                if (vLimit > iHighLimit) vLimit = iHighLimit;
                mlt = (int)LZ4_count(ip+MINMATCH, matchPtr+MINMATCH, vLimit) + MINMATCH;
                back = lookBackLength ? LZ4HC_countBack(ip, matchPtr, iLowLimit, dictCtx->prefixStart) : 0;
                mlt -= back;
                if (mlt > longest) {
                    longest = mlt;
                    *matchpos = prefixPtr - prefixIdx + matchIndex + back;
                    *startpos = ip + back;
            }   }

            {   U32 const nextOffset = DELTANEXTU16(dictCtx->chainTable, dictMatchIndex);
                dictMatchIndex -= nextOffset;
                matchIndex -= nextOffset;
    }   }   }

    return longest;
}

LZ4_FORCE_INLINE int
LZ4HC_InsertAndFindBestMatch(LZ4HC_CCtx_internal* const hc4,   /* Index table will be updated */
                       const BYTE* const ip, const BYTE* const iLimit,
                       const BYTE** matchpos,
                       const int maxNbAttempts,
                       const int patternAnalysis,
                       const dictCtx_directive dict)
{
    const BYTE* uselessPtr = ip;
    /* note : LZ4HC_InsertAndGetWiderMatch() is able to modify the starting position of a match (*startpos),
     * but this won't be the case here, as we define iLowLimit==ip,
     * so LZ4HC_InsertAndGetWiderMatch() won't be allowed to search past ip */
    return LZ4HC_InsertAndGetWiderMatch(hc4, ip, ip, iLimit, MINMATCH-1, matchpos, &uselessPtr, maxNbAttempts, patternAnalysis, 0 /*chainSwap*/, dict, favorCompressionRatio);
}

/* LZ4HC_encodeSequence() :
 * @return : 0 if ok,
 *           1 if buffer issue detected */
LZ4_FORCE_INLINE int LZ4HC_encodeSequence (
    const BYTE** _ip,
    BYTE** _op,
    const BYTE** _anchor,
    int matchLength,
    const BYTE* const match,
    limitedOutput_directive limit,
    BYTE* oend)
{
#define ip      (*_ip)
#define op      (*_op)
#define anchor  (*_anchor)

    size_t length;
    BYTE* const token = op++;

#if defined(LZ4_DEBUG) && (LZ4_DEBUG >= 6)
    static const BYTE* start = NULL;
    static U32 totalCost = 0;
    U32 const pos = (start==NULL) ? 0 : (U32)(anchor - start);
    U32 const ll = (U32)(ip - anchor);
    U32 const llAdd = (ll>=15) ? ((ll-15) / 255) + 1 : 0;
    U32 const mlAdd = (matchLength>=19) ? ((matchLength-19) / 255) + 1 : 0;
    U32 const cost = 1 + llAdd + ll + 2 + mlAdd;
    if (start==NULL) start = anchor;  /* only works for single segment */
    /* g_debuglog_enable = (pos >= 2228) & (pos <= 2262); */
    DEBUGLOG(6, "pos:%7u -- literals:%4u, match:%4i, offset:%5u, cost:%4u + %5u",
                pos,
                (U32)(ip - anchor), matchLength, (U32)(ip-match),
                cost, totalCost);
    totalCost += cost;
#endif

    /* Encode Literal length */
    length = (size_t)(ip - anchor);
    LZ4_STATIC_ASSERT(notLimited == 0);
    /* Check output limit */
    if (limit && ((op + (length / 255) + length + (2 + 1 + LASTLITERALS)) > oend)) {
        DEBUGLOG(6, "Not enough room to write %i literals (%i bytes remaining)",
                (int)length, (int)(oend - op));
        return 1;
    }
    if (length >= RUN_MASK) {
        size_t len = length - RUN_MASK;
        *token = (RUN_MASK << ML_BITS);
        for(; len >= 255 ; len -= 255) *op++ = 255;
        *op++ = (BYTE)len;
    } else {
        *token = (BYTE)(length << ML_BITS);
    }

    /* Copy Literals */
    LZ4_wildCopy8(op, anchor, op + length);
    op += length;

    /* Encode Offset */
    assert( (ip - match) <= LZ4_DISTANCE_MAX );   /* note : consider providing offset as a value, rather than as a pointer difference */
    LZ4_writeLE16(op, (U16)(ip - match)); op += 2;

    /* Encode MatchLength */
    assert(matchLength >= MINMATCH);
    length = (size_t)matchLength - MINMATCH;
    if (limit && (op + (length / 255) + (1 + LASTLITERALS) > oend)) {
        DEBUGLOG(6, "Not enough room to write match length");
        return 1;   /* Check output limit */
    }
    if (length >= ML_MASK) {
        *token += ML_MASK;
        length -= ML_MASK;
        for(; length >= 510 ; length -= 510) { *op++ = 255; *op++ = 255; }
        if (length >= 255) { length -= 255; *op++ = 255; }
        *op++ = (BYTE)length;
    } else {
        *token += (BYTE)(length);
    }

    /* Prepare next loop */
    ip += matchLength;
    anchor = ip;

    return 0;
}
#undef ip
#undef op
#undef anchor

LZ4_FORCE_INLINE int LZ4HC_compress_hashChain (
    LZ4HC_CCtx_internal* const ctx,
    const char* const source,
    char* const dest,
    int* srcSizePtr,
    int const maxOutputSize,
    int maxNbAttempts,
    const limitedOutput_directive limit,
    const dictCtx_directive dict
    )
{
    const int inputSize = *srcSizePtr;
    const int patternAnalysis = (maxNbAttempts > 128);   /* levels 9+ */

    const BYTE* ip = (const BYTE*) source;
    const BYTE* anchor = ip;
    const BYTE* const iend = ip + inputSize;
    const BYTE* const mflimit = iend - MFLIMIT;
    const BYTE* const matchlimit = (iend - LASTLITERALS);

    BYTE* optr = (BYTE*) dest;
    BYTE* op = (BYTE*) dest;
    BYTE* oend = op + maxOutputSize;

    int   ml0, ml, ml2, ml3;
    const BYTE* start0;
    const BYTE* ref0;
    const BYTE* ref = NULL;
    const BYTE* start2 = NULL;
    const BYTE* ref2 = NULL;
    const BYTE* start3 = NULL;
    const BYTE* ref3 = NULL;

    /* init */
    *srcSizePtr = 0;
    if (limit == fillOutput) oend -= LASTLITERALS;                  /* Hack for support LZ4 format restriction */
    if (inputSize < LZ4_minLength) goto _last_literals;             /* Input too small, no compression (all literals) */

    /* Main Loop */
    while (ip <= mflimit) {
        ml = LZ4HC_InsertAndFindBestMatch(ctx, ip, matchlimit, &ref, maxNbAttempts, patternAnalysis, dict);
        if (ml<MINMATCH) { ip++; continue; }

        /* saved, in case we would skip too much */
        start0 = ip; ref0 = ref; ml0 = ml;

_Search2:
        if (ip+ml <= mflimit) {
            ml2 = LZ4HC_InsertAndGetWiderMatch(ctx,
                            ip + ml - 2, ip + 0, matchlimit, ml, &ref2, &start2,
                            maxNbAttempts, patternAnalysis, 0, dict, favorCompressionRatio);
        } else {
            ml2 = ml;
        }

        if (ml2 == ml) { /* No better match => encode ML1 */
            optr = op;
            if (LZ4HC_encodeSequence(UPDATABLE(ip, op, anchor), ml, ref, limit, oend)) goto _dest_overflow;
            continue;
        }

        if (start0 < ip) {   /* first match was skipped at least once */
            if (start2 < ip + ml0) {  /* squeezing ML1 between ML0(original ML1) and ML2 */
                ip = start0; ref = ref0; ml = ml0;  /* restore initial ML1 */
        }   }

        /* Here, start0==ip */
        if ((start2 - ip) < 3) {  /* First Match too small : removed */
            ml = ml2;
            ip = start2;
            ref =ref2;
            goto _Search2;
        }

_Search3:
        /* At this stage, we have :
        *  ml2 > ml1, and
        *  ip1+3 <= ip2 (usually < ip1+ml1) */
        if ((start2 - ip) < OPTIMAL_ML) {
            int correction;
            int new_ml = ml;
            if (new_ml > OPTIMAL_ML) new_ml = OPTIMAL_ML;
            if (ip+new_ml > start2 + ml2 - MINMATCH) new_ml = (int)(start2 - ip) + ml2 - MINMATCH;
            correction = new_ml - (int)(start2 - ip);
            if (correction > 0) {
                start2 += correction;
                ref2 += correction;
                ml2 -= correction;
            }
        }
        /* Now, we have start2 = ip+new_ml, with new_ml = min(ml, OPTIMAL_ML=18) */

        if (start2 + ml2 <= mflimit) {
            ml3 = LZ4HC_InsertAndGetWiderMatch(ctx,
                            start2 + ml2 - 3, start2, matchlimit, ml2, &ref3, &start3,
                            maxNbAttempts, patternAnalysis, 0, dict, favorCompressionRatio);
        } else {
            ml3 = ml2;
        }

        if (ml3 == ml2) {  /* No better match => encode ML1 and ML2 */
            /* ip & ref are known; Now for ml */
            if (start2 < ip+ml)  ml = (int)(start2 - ip);
            /* Now, encode 2 sequences */
            optr = op;
            if (LZ4HC_encodeSequence(UPDATABLE(ip, op, anchor), ml, ref, limit, oend)) goto _dest_overflow;
            ip = start2;
            optr = op;
            if (LZ4HC_encodeSequence(UPDATABLE(ip, op, anchor), ml2, ref2, limit, oend)) {
                ml  = ml2;
                ref = ref2;
                goto _dest_overflow;
            }
            continue;
        }

        if (start3 < ip+ml+3) {  /* Not enough space for match 2 : remove it */
            if (start3 >= (ip+ml)) {  /* can write Seq1 immediately ==> Seq2 is removed, so Seq3 becomes Seq1 */
                if (start2 < ip+ml) {
                    int correction = (int)(ip+ml - start2);
                    start2 += correction;
                    ref2 += correction;
                    ml2 -= correction;
                    if (ml2 < MINMATCH) {
                        start2 = start3;
                        ref2 = ref3;
                        ml2 = ml3;
                    }
                }

                optr = op;
                if (LZ4HC_encodeSequence(UPDATABLE(ip, op, anchor), ml, ref, limit, oend)) goto _dest_overflow;
                ip  = start3;
                ref = ref3;
                ml  = ml3;

                start0 = start2;
                ref0 = ref2;
                ml0 = ml2;
                goto _Search2;
            }

            start2 = start3;
            ref2 = ref3;
            ml2 = ml3;
            goto _Search3;
        }

        /*
        * OK, now we have 3 ascending matches;
        * let's write the first one ML1.
        * ip & ref are known; Now decide ml.
        */
        if (start2 < ip+ml) {
            if ((start2 - ip) < OPTIMAL_ML) {
                int correction;
                if (ml > OPTIMAL_ML) ml = OPTIMAL_ML;
                if (ip + ml > start2 + ml2 - MINMATCH) ml = (int)(start2 - ip) + ml2 - MINMATCH;
                correction = ml - (int)(start2 - ip);
                if (correction > 0) {
                    start2 += correction;
                    ref2 += correction;
                    ml2 -= correction;
                }
            } else {
                ml = (int)(start2 - ip);
            }
        }
        optr = op;
        if (LZ4HC_encodeSequence(UPDATABLE(ip, op, anchor), ml, ref, limit, oend)) goto _dest_overflow;

        /* ML2 becomes ML1 */
        ip = start2; ref = ref2; ml = ml2;

        /* ML3 becomes ML2 */
        start2 = start3; ref2 = ref3; ml2 = ml3;

        /* let's find a new ML3 */
        goto _Search3;
    }

_last_literals:
    /* Encode Last Literals */
    {   size_t lastRunSize = (size_t)(iend - anchor);  /* literals */
        size_t llAdd = (lastRunSize + 255 - RUN_MASK) / 255;
        size_t const totalSize = 1 + llAdd + lastRunSize;
        if (limit == fillOutput) oend += LASTLITERALS;  /* restore correct value */
        if (limit && (op + totalSize > oend)) {
            if (limit == limitedOutput) return 0;
            /* adapt lastRunSize to fill 'dest' */
            lastRunSize  = (size_t)(oend - op) - 1 /*token*/;
            llAdd = (lastRunSize + 256 - RUN_MASK) / 256;
            lastRunSize -= llAdd;
        }
        DEBUGLOG(6, "Final literal run : %i literals", (int)lastRunSize);
        ip = anchor + lastRunSize;  /* can be != iend if limit==fillOutput */

        if (lastRunSize >= RUN_MASK) {
            size_t accumulator = lastRunSize - RUN_MASK;
            *op++ = (RUN_MASK << ML_BITS);
            for(; accumulator >= 255 ; accumulator -= 255) *op++ = 255;
            *op++ = (BYTE) accumulator;
        } else {
            *op++ = (BYTE)(lastRunSize << ML_BITS);
        }
        LZ4_memcpy(op, anchor, lastRunSize);
        op += lastRunSize;
    }

    /* End */
    *srcSizePtr = (int) (((const char*)ip) - source);
    return (int) (((char*)op)-dest);

_dest_overflow:
    if (limit == fillOutput) {
        /* Assumption : ip, anchor, ml and ref must be set correctly */
        size_t const ll = (size_t)(ip - anchor);
        size_t const ll_addbytes = (ll + 240) / 255;
        size_t const ll_totalCost = 1 + ll_addbytes + ll;
        BYTE* const maxLitPos = oend - 3; /* 2 for offset, 1 for token */
        DEBUGLOG(6, "Last sequence overflowing");
        op = optr;  /* restore correct out pointer */
        if (op + ll_totalCost <= maxLitPos) {
            /* ll validated; now adjust match length */
            size_t const bytesLeftForMl = (size_t)(maxLitPos - (op+ll_totalCost));
            size_t const maxMlSize = MINMATCH + (ML_MASK-1) + (bytesLeftForMl * 255);
            assert(maxMlSize < INT_MAX); assert(ml >= 0);
            if ((size_t)ml > maxMlSize) ml = (int)maxMlSize;
            if ((oend + LASTLITERALS) - (op + ll_totalCost + 2) - 1 + ml >= MFLIMIT) {
                LZ4HC_encodeSequence(UPDATABLE(ip, op, anchor), ml, ref, notLimited, oend);
        }   }
        goto _last_literals;
    }
    /* compression failed */
    return 0;
}


static int LZ4HC_compress_optimal( LZ4HC_CCtx_internal* ctx,
    const char* const source, char* dst,
    int* srcSizePtr, int dstCapacity,
    int const nbSearches, size_t sufficient_len,
    const limitedOutput_directive limit, int const fullUpdate,
    const dictCtx_directive dict,
    const HCfavor_e favorDecSpeed);


LZ4_FORCE_INLINE int LZ4HC_compress_generic_internal (
    LZ4HC_CCtx_internal* const ctx,
    const char* const src,
    char* const dst,
    int* const srcSizePtr,
    int const dstCapacity,
    int cLevel,
    const limitedOutput_directive limit,
    const dictCtx_directive dict
    )
{
    typedef enum { lz4hc, lz4opt } lz4hc_strat_e;
    typedef struct {
        lz4hc_strat_e strat;
        int nbSearches;
        U32 targetLength;
    } cParams_t;
    static const cParams_t clTable[LZ4HC_CLEVEL_MAX+1] = {
        { lz4hc,     2, 16 },  /* 0, unused */
        { lz4hc,     2, 16 },  /* 1, unused */
        { lz4hc,     2, 16 },  /* 2, unused */
        { lz4hc,     4, 16 },  /* 3 */
        { lz4hc,     8, 16 },  /* 4 */
        { lz4hc,    16, 16 },  /* 5 */
        { lz4hc,    32, 16 },  /* 6 */
        { lz4hc,    64, 16 },  /* 7 */
        { lz4hc,   128, 16 },  /* 8 */
        { lz4hc,   256, 16 },  /* 9 */
        { lz4opt,   96, 64 },  /*10==LZ4HC_CLEVEL_OPT_MIN*/
        { lz4opt,  512,128 },  /*11 */
        { lz4opt,16384,LZ4_OPT_NUM },  /* 12==LZ4HC_CLEVEL_MAX */
    };

    DEBUGLOG(4, "LZ4HC_compress_generic(ctx=%p, src=%p, srcSize=%d, limit=%d)",
                ctx, src, *srcSizePtr, limit);

    if (limit == fillOutput && dstCapacity < 1) return 0;   /* Impossible to store anything */
    if ((U32)*srcSizePtr > (U32)LZ4_MAX_INPUT_SIZE) return 0;    /* Unsupported input size (too large or negative) */

    ctx->end += *srcSizePtr;
    if (cLevel < 1) cLevel = LZ4HC_CLEVEL_DEFAULT;   /* note : convention is different from lz4frame, maybe something to review */
    cLevel = lz4Min(LZ4HC_CLEVEL_MAX, cLevel);
    {   cParams_t const cParam = clTable[cLevel];
        HCfavor_e const favor = ctx->favorDecSpeed ? favorDecompressionSpeed : favorCompressionRatio;
        int result;

        if (cParam.strat == lz4hc) {
            result = LZ4HC_compress_hashChain(ctx,
                                src, dst, srcSizePtr, dstCapacity,
                                cParam.nbSearches, limit, dict);
        } else {
            assert(cParam.strat == lz4opt);
            result = LZ4HC_compress_optimal(ctx,
                                src, dst, srcSizePtr, dstCapacity,
                                cParam.nbSearches, cParam.targetLength, limit,
                                cLevel == LZ4HC_CLEVEL_MAX,   /* ultra mode */
                                dict, favor);
        }
        if (result <= 0) ctx->dirty = 1;
        return result;
    }
}

static void LZ4HC_setExternalDict(LZ4HC_CCtx_internal* ctxPtr, const BYTE* newBlock);

static int
LZ4HC_compress_generic_noDictCtx (
        LZ4HC_CCtx_internal* const ctx,
        const char* const src,
        char* const dst,
        int* const srcSizePtr,
        int const dstCapacity,
        int cLevel,
        limitedOutput_directive limit
        )
{
    assert(ctx->dictCtx == NULL);
    return LZ4HC_compress_generic_internal(ctx, src, dst, srcSizePtr, dstCapacity, cLevel, limit, noDictCtx);
}

static int
LZ4HC_compress_generic_dictCtx (
        LZ4HC_CCtx_internal* const ctx,
        const char* const src,
        char* const dst,
        int* const srcSizePtr,
        int const dstCapacity,
        int cLevel,
        limitedOutput_directive limit
        )
{
    const size_t position = (size_t)(ctx->end - ctx->prefixStart) + (ctx->dictLimit - ctx->lowLimit);
    assert(ctx->dictCtx != NULL);
    if (position >= 64 KB) {
        ctx->dictCtx = NULL;
        return LZ4HC_compress_generic_noDictCtx(ctx, src, dst, srcSizePtr, dstCapacity, cLevel, limit);
    } else if (position == 0 && *srcSizePtr > 4 KB) {
        LZ4_memcpy(ctx, ctx->dictCtx, sizeof(LZ4HC_CCtx_internal));
        LZ4HC_setExternalDict(ctx, (const BYTE *)src);
        ctx->compressionLevel = (short)cLevel;
        return LZ4HC_compress_generic_noDictCtx(ctx, src, dst, srcSizePtr, dstCapacity, cLevel, limit);
    } else {
        return LZ4HC_compress_generic_internal(ctx, src, dst, srcSizePtr, dstCapacity, cLevel, limit, usingDictCtxHc);
    }
}

static int
LZ4HC_compress_generic (
        LZ4HC_CCtx_internal* const ctx,
        const char* const src,
        char* const dst,
        int* const srcSizePtr,
        int const dstCapacity,
        int cLevel,
        limitedOutput_directive limit
        )
{
    if (ctx->dictCtx == NULL) {
        return LZ4HC_compress_generic_noDictCtx(ctx, src, dst, srcSizePtr, dstCapacity, cLevel, limit);
    } else {
        return LZ4HC_compress_generic_dictCtx(ctx, src, dst, srcSizePtr, dstCapacity, cLevel, limit);
    }
}


int LZ4_sizeofStateHC(void) { return (int)sizeof(LZ4_streamHC_t); }

static size_t LZ4_streamHC_t_alignment(void)
{
#if LZ4_ALIGN_TEST
    typedef struct { char c; LZ4_streamHC_t t; } t_a;
    return sizeof(t_a) - sizeof(LZ4_streamHC_t);
#else
    return 1;  /* effectively disabled */
#endif
}

/* state is presumed correctly initialized,
 * in which case its size and alignment have already been validate */
int LZ4_compress_HC_extStateHC_fastReset (void* state, const char* src, char* dst, int srcSize, int dstCapacity, int compressionLevel)
{
    LZ4HC_CCtx_internal* const ctx = &((LZ4_streamHC_t*)state)->internal_donotuse;
    if (!LZ4_isAligned(state, LZ4_streamHC_t_alignment())) return 0;
    LZ4_resetStreamHC_fast((LZ4_streamHC_t*)state, compressionLevel);
    LZ4HC_init_internal (ctx, (const BYTE*)src);
    if (dstCapacity < LZ4_compressBound(srcSize))
        return LZ4HC_compress_generic (ctx, src, dst, &srcSize, dstCapacity, compressionLevel, limitedOutput);
    else
        return LZ4HC_compress_generic (ctx, src, dst, &srcSize, dstCapacity, compressionLevel, notLimited);
}

int LZ4_compress_HC_extStateHC (void* state, const char* src, char* dst, int srcSize, int dstCapacity, int compressionLevel)
{
    LZ4_streamHC_t* const ctx = LZ4_initStreamHC(state, sizeof(*ctx));
    if (ctx==NULL) return 0;   /* init failure */
    return LZ4_compress_HC_extStateHC_fastReset(state, src, dst, srcSize, dstCapacity, compressionLevel);
}

int LZ4_compress_HC(const char* src, char* dst, int srcSize, int dstCapacity, int compressionLevel)
{
    int cSize;
#if defined(LZ4HC_HEAPMODE) && LZ4HC_HEAPMODE==1
    LZ4_streamHC_t* const statePtr = (LZ4_streamHC_t*)ALLOC(sizeof(LZ4_streamHC_t));
    if (statePtr==NULL) return 0;
#else
    LZ4_streamHC_t state;
    LZ4_streamHC_t* const statePtr = &state;
#endif
    cSize = LZ4_compress_HC_extStateHC(statePtr, src, dst, srcSize, dstCapacity, compressionLevel);
#if defined(LZ4HC_HEAPMODE) && LZ4HC_HEAPMODE==1
    FREEMEM(statePtr);
#endif
    return cSize;
}

/* state is presumed sized correctly (>= sizeof(LZ4_streamHC_t)) */
int LZ4_compress_HC_destSize(void* state, const char* source, char* dest, int* sourceSizePtr, int targetDestSize, int cLevel)
{
    LZ4_streamHC_t* const ctx = LZ4_initStreamHC(state, sizeof(*ctx));
    if (ctx==NULL) return 0;   /* init failure */
    LZ4HC_init_internal(&ctx->internal_donotuse, (const BYTE*) source);
    LZ4_setCompressionLevel(ctx, cLevel);
    return LZ4HC_compress_generic(&ctx->internal_donotuse, source, dest, sourceSizePtr, targetDestSize, cLevel, fillOutput);
}



/**************************************
*  Streaming Functions
**************************************/
/* allocation */
#if !defined(LZ4_STATIC_LINKING_ONLY_DISABLE_MEMORY_ALLOCATION)
LZ4_streamHC_t* LZ4_createStreamHC(void)
{
    LZ4_streamHC_t* const state =
        (LZ4_streamHC_t*)ALLOC_AND_ZERO(sizeof(LZ4_streamHC_t));
    if (state == NULL) return NULL;
    LZ4_setCompressionLevel(state, LZ4HC_CLEVEL_DEFAULT);
    return state;
}

int LZ4_freeStreamHC (LZ4_streamHC_t* LZ4_streamHCPtr)
{
    DEBUGLOG(4, "LZ4_freeStreamHC(%p)", LZ4_streamHCPtr);
    if (!LZ4_streamHCPtr) return 0;  /* support free on NULL */
    FREEMEM(LZ4_streamHCPtr);
    return 0;
}
#endif


LZ4_streamHC_t* LZ4_initStreamHC (void* buffer, size_t size)
{
    LZ4_streamHC_t* const LZ4_streamHCPtr = (LZ4_streamHC_t*)buffer;
    DEBUGLOG(4, "LZ4_initStreamHC(%p, %u)", buffer, (unsigned)size);
    /* check conditions */
    if (buffer == NULL) return NULL;
    if (size < sizeof(LZ4_streamHC_t)) return NULL;
    if (!LZ4_isAligned(buffer, LZ4_streamHC_t_alignment())) return NULL;
    /* init */
    { LZ4HC_CCtx_internal* const hcstate = &(LZ4_streamHCPtr->internal_donotuse);
      MEM_INIT(hcstate, 0, sizeof(*hcstate)); }
    LZ4_setCompressionLevel(LZ4_streamHCPtr, LZ4HC_CLEVEL_DEFAULT);
    return LZ4_streamHCPtr;
}

/* just a stub */
void LZ4_resetStreamHC (LZ4_streamHC_t* LZ4_streamHCPtr, int compressionLevel)
{
    LZ4_initStreamHC(LZ4_streamHCPtr, sizeof(*LZ4_streamHCPtr));
    LZ4_setCompressionLevel(LZ4_streamHCPtr, compressionLevel);
}

void LZ4_resetStreamHC_fast (LZ4_streamHC_t* LZ4_streamHCPtr, int compressionLevel)
{
    DEBUGLOG(4, "LZ4_resetStreamHC_fast(%p, %d)", LZ4_streamHCPtr, compressionLevel);
    if (LZ4_streamHCPtr->internal_donotuse.dirty) {
        LZ4_initStreamHC(LZ4_streamHCPtr, sizeof(*LZ4_streamHCPtr));
    } else {
        /* preserve end - prefixStart : can trigger clearTable's threshold */
        if (LZ4_streamHCPtr->internal_donotuse.end != NULL) {
            LZ4_streamHCPtr->internal_donotuse.end -= (uptrval)LZ4_streamHCPtr->internal_donotuse.prefixStart;
        } else {
            assert(LZ4_streamHCPtr->internal_donotuse.prefixStart == NULL);
        }
        LZ4_streamHCPtr->internal_donotuse.prefixStart = NULL;
        LZ4_streamHCPtr->internal_donotuse.dictCtx = NULL;
    }
    LZ4_setCompressionLevel(LZ4_streamHCPtr, compressionLevel);
}

void LZ4_setCompressionLevel(LZ4_streamHC_t* LZ4_streamHCPtr, int compressionLevel)
{
    DEBUGLOG(5, "LZ4_setCompressionLevel(%p, %d)", LZ4_streamHCPtr, compressionLevel);
    if (compressionLevel < 1) compressionLevel = LZ4HC_CLEVEL_DEFAULT;
    if (compressionLevel > LZ4HC_CLEVEL_MAX) compressionLevel = LZ4HC_CLEVEL_MAX;
    LZ4_streamHCPtr->internal_donotuse.compressionLevel = (short)compressionLevel;
}

void LZ4_favorDecompressionSpeed(LZ4_streamHC_t* LZ4_streamHCPtr, int favor)
{
    LZ4_streamHCPtr->internal_donotuse.favorDecSpeed = (favor!=0);
}

/* LZ4_loadDictHC() :
 * LZ4_streamHCPtr is presumed properly initialized */
int LZ4_loadDictHC (LZ4_streamHC_t* LZ4_streamHCPtr,
              const char* dictionary, int dictSize)
{
    LZ4HC_CCtx_internal* const ctxPtr = &LZ4_streamHCPtr->internal_donotuse;
    DEBUGLOG(4, "LZ4_loadDictHC(ctx:%p, dict:%p, dictSize:%d)", LZ4_streamHCPtr, dictionary, dictSize);
    assert(LZ4_streamHCPtr != NULL);
    if (dictSize > 64 KB) {
        dictionary += (size_t)dictSize - 64 KB;
        dictSize = 64 KB;
    }
    /* need a full initialization, there are bad side-effects when using resetFast() */
    {   int const cLevel = ctxPtr->compressionLevel;
        LZ4_initStreamHC(LZ4_streamHCPtr, sizeof(*LZ4_streamHCPtr));
        LZ4_setCompressionLevel(LZ4_streamHCPtr, cLevel);
    }
    LZ4HC_init_internal (ctxPtr, (const BYTE*)dictionary);
    ctxPtr->end = (const BYTE*)dictionary + dictSize;
    if (dictSize >= 4) LZ4HC_Insert (ctxPtr, ctxPtr->end-3);
    return dictSize;
}

void LZ4_attach_HC_dictionary(LZ4_streamHC_t *working_stream, const LZ4_streamHC_t *dictionary_stream) {
    working_stream->internal_donotuse.dictCtx = dictionary_stream != NULL ? &(dictionary_stream->internal_donotuse) : NULL;
}

/* compression */

static void LZ4HC_setExternalDict(LZ4HC_CCtx_internal* ctxPtr, const BYTE* newBlock)
{
    DEBUGLOG(4, "LZ4HC_setExternalDict(%p, %p)", ctxPtr, newBlock);
    if (ctxPtr->end >= ctxPtr->prefixStart + 4)
        LZ4HC_Insert (ctxPtr, ctxPtr->end-3);   /* Referencing remaining dictionary content */

    /* Only one memory segment for extDict, so any previous extDict is lost at this stage */
    ctxPtr->lowLimit  = ctxPtr->dictLimit;
    ctxPtr->dictStart  = ctxPtr->prefixStart;
    ctxPtr->dictLimit += (U32)(ctxPtr->end - ctxPtr->prefixStart);
    ctxPtr->prefixStart = newBlock;
    ctxPtr->end  = newBlock;
    ctxPtr->nextToUpdate = ctxPtr->dictLimit;   /* match referencing will resume from there */

    /* cannot reference an extDict and a dictCtx at the same time */
    ctxPtr->dictCtx = NULL;
}

static int
LZ4_compressHC_continue_generic (LZ4_streamHC_t* LZ4_streamHCPtr,
                                 const char* src, char* dst,
                                 int* srcSizePtr, int dstCapacity,
                                 limitedOutput_directive limit)
{
    LZ4HC_CCtx_internal* const ctxPtr = &LZ4_streamHCPtr->internal_donotuse;
    DEBUGLOG(5, "LZ4_compressHC_continue_generic(ctx=%p, src=%p, srcSize=%d, limit=%d)",
                LZ4_streamHCPtr, src, *srcSizePtr, limit);
    assert(ctxPtr != NULL);
    /* auto-init if forgotten */
    if (ctxPtr->prefixStart == NULL) LZ4HC_init_internal (ctxPtr, (const BYTE*) src);

    /* Check overflow */
    if ((size_t)(ctxPtr->end - ctxPtr->prefixStart) + ctxPtr->dictLimit > 2 GB) {
        size_t dictSize = (size_t)(ctxPtr->end - ctxPtr->prefixStart);
        if (dictSize > 64 KB) dictSize = 64 KB;
        LZ4_loadDictHC(LZ4_streamHCPtr, (const char*)(ctxPtr->end) - dictSize, (int)dictSize);
    }

    /* Check if blocks follow each other */
    if ((const BYTE*)src != ctxPtr->end)
        LZ4HC_setExternalDict(ctxPtr, (const BYTE*)src);

    /* Check overlapping input/dictionary space */
    {   const BYTE* sourceEnd = (const BYTE*) src + *srcSizePtr;
        const BYTE* const dictBegin = ctxPtr->dictStart;
        const BYTE* const dictEnd   = ctxPtr->dictStart + (ctxPtr->dictLimit - ctxPtr->lowLimit);
        if ((sourceEnd > dictBegin) && ((const BYTE*)src < dictEnd)) {
            if (sourceEnd > dictEnd) sourceEnd = dictEnd;
            ctxPtr->lowLimit += (U32)(sourceEnd - ctxPtr->dictStart);
            ctxPtr->dictStart += (U32)(sourceEnd - ctxPtr->dictStart);
            if (ctxPtr->dictLimit - ctxPtr->lowLimit < 4) {
                ctxPtr->lowLimit = ctxPtr->dictLimit;
                ctxPtr->dictStart = ctxPtr->prefixStart;
    }   }   }

    return LZ4HC_compress_generic (ctxPtr, src, dst, srcSizePtr, dstCapacity, ctxPtr->compressionLevel, limit);
}

int LZ4_compress_HC_continue (LZ4_streamHC_t* LZ4_streamHCPtr, const char* src, char* dst, int srcSize, int dstCapacity)
{
    if (dstCapacity < LZ4_compressBound(srcSize))
        return LZ4_compressHC_continue_generic (LZ4_streamHCPtr, src, dst, &srcSize, dstCapacity, limitedOutput);
    else
        return LZ4_compressHC_continue_generic (LZ4_streamHCPtr, src, dst, &srcSize, dstCapacity, notLimited);
}

int LZ4_compress_HC_continue_destSize (LZ4_streamHC_t* LZ4_streamHCPtr, const char* src, char* dst, int* srcSizePtr, int targetDestSize)
{
    return LZ4_compressHC_continue_generic(LZ4_streamHCPtr, src, dst, srcSizePtr, targetDestSize, fillOutput);
}



/* LZ4_saveDictHC :
 * save history content
 * into a user-provided buffer
 * which is then used to continue compression
 */
int LZ4_saveDictHC (LZ4_streamHC_t* LZ4_streamHCPtr, char* safeBuffer, int dictSize)
{
    LZ4HC_CCtx_internal* const streamPtr = &LZ4_streamHCPtr->internal_donotuse;
    int const prefixSize = (int)(streamPtr->end - streamPtr->prefixStart);
    DEBUGLOG(5, "LZ4_saveDictHC(%p, %p, %d)", LZ4_streamHCPtr, safeBuffer, dictSize);
    assert(prefixSize >= 0);
    if (dictSize > 64 KB) dictSize = 64 KB;
    if (dictSize < 4) dictSize = 0;
    if (dictSize > prefixSize) dictSize = prefixSize;
    if (safeBuffer == NULL) assert(dictSize == 0);
    if (dictSize > 0)
        LZ4_memmove(safeBuffer, streamPtr->end - dictSize, dictSize);
    {   U32 const endIndex = (U32)(streamPtr->end - streamPtr->prefixStart) + streamPtr->dictLimit;
        streamPtr->end = (const BYTE*)safeBuffer + dictSize;
        streamPtr->prefixStart = streamPtr->end - dictSize;
        streamPtr->dictLimit = endIndex - (U32)dictSize;
        streamPtr->lowLimit = endIndex - (U32)dictSize;
        streamPtr->dictStart = streamPtr->prefixStart;
        if (streamPtr->nextToUpdate < streamPtr->dictLimit)
            streamPtr->nextToUpdate = streamPtr->dictLimit;
    }
    return dictSize;
}


/***************************************************
*  Deprecated Functions
***************************************************/

/* These functions currently generate deprecation warnings */

/* Wrappers for deprecated compression functions */
int LZ4_compressHC(const char* src, char* dst, int srcSize) { return LZ4_compress_HC (src, dst, srcSize, LZ4_compressBound(srcSize), 0); }
int LZ4_compressHC_limitedOutput(const char* src, char* dst, int srcSize, int maxDstSize) { return LZ4_compress_HC(src, dst, srcSize, maxDstSize, 0); }
int LZ4_compressHC2(const char* src, char* dst, int srcSize, int cLevel) { return LZ4_compress_HC (src, dst, srcSize, LZ4_compressBound(srcSize), cLevel); }
int LZ4_compressHC2_limitedOutput(const char* src, char* dst, int srcSize, int maxDstSize, int cLevel) { return LZ4_compress_HC(src, dst, srcSize, maxDstSize, cLevel); }
int LZ4_compressHC_withStateHC (void* state, const char* src, char* dst, int srcSize) { return LZ4_compress_HC_extStateHC (state, src, dst, srcSize, LZ4_compressBound(srcSize), 0); }
int LZ4_compressHC_limitedOutput_withStateHC (void* state, const char* src, char* dst, int srcSize, int maxDstSize) { return LZ4_compress_HC_extStateHC (state, src, dst, srcSize, maxDstSize, 0); }
int LZ4_compressHC2_withStateHC (void* state, const char* src, char* dst, int srcSize, int cLevel) { return LZ4_compress_HC_extStateHC(state, src, dst, srcSize, LZ4_compressBound(srcSize), cLevel); }
int LZ4_compressHC2_limitedOutput_withStateHC (void* state, const char* src, char* dst, int srcSize, int maxDstSize, int cLevel) { return LZ4_compress_HC_extStateHC(state, src, dst, srcSize, maxDstSize, cLevel); }
int LZ4_compressHC_continue (LZ4_streamHC_t* ctx, const char* src, char* dst, int srcSize) { return LZ4_compress_HC_continue (ctx, src, dst, srcSize, LZ4_compressBound(srcSize)); }
int LZ4_compressHC_limitedOutput_continue (LZ4_streamHC_t* ctx, const char* src, char* dst, int srcSize, int maxDstSize) { return LZ4_compress_HC_continue (ctx, src, dst, srcSize, maxDstSize); }


/* Deprecated streaming functions */
int LZ4_sizeofStreamStateHC(void) { return sizeof(LZ4_streamHC_t); }

/* state is presumed correctly sized, aka >= sizeof(LZ4_streamHC_t)
 * @return : 0 on success, !=0 if error */
int LZ4_resetStreamStateHC(void* state, char* inputBuffer)
{
    LZ4_streamHC_t* const hc4 = LZ4_initStreamHC(state, sizeof(*hc4));
    if (hc4 == NULL) return 1;   /* init failed */
    LZ4HC_init_internal (&hc4->internal_donotuse, (const BYTE*)inputBuffer);
    return 0;
}

#if !defined(LZ4_STATIC_LINKING_ONLY_DISABLE_MEMORY_ALLOCATION)
void* LZ4_createHC (const char* inputBuffer)
{
    LZ4_streamHC_t* const hc4 = LZ4_createStreamHC();
    if (hc4 == NULL) return NULL;   /* not enough memory */
    LZ4HC_init_internal (&hc4->internal_donotuse, (const BYTE*)inputBuffer);
    return hc4;
}

int LZ4_freeHC (void* LZ4HC_Data)
{
    if (!LZ4HC_Data) return 0;  /* support free on NULL */
    FREEMEM(LZ4HC_Data);
    return 0;
}
#endif

int LZ4_compressHC2_continue (void* LZ4HC_Data, const char* src, char* dst, int srcSize, int cLevel)
{
    return LZ4HC_compress_generic (&((LZ4_streamHC_t*)LZ4HC_Data)->internal_donotuse, src, dst, &srcSize, 0, cLevel, notLimited);
}

int LZ4_compressHC2_limitedOutput_continue (void* LZ4HC_Data, const char* src, char* dst, int srcSize, int dstCapacity, int cLevel)
{
    return LZ4HC_compress_generic (&((LZ4_streamHC_t*)LZ4HC_Data)->internal_donotuse, src, dst, &srcSize, dstCapacity, cLevel, limitedOutput);
}

char* LZ4_slideInputBufferHC(void* LZ4HC_Data)
{
    LZ4_streamHC_t* const ctx = (LZ4_streamHC_t*)LZ4HC_Data;
    const BYTE* bufferStart = ctx->internal_donotuse.prefixStart - ctx->internal_donotuse.dictLimit + ctx->internal_donotuse.lowLimit;
    LZ4_resetStreamHC_fast(ctx, ctx->internal_donotuse.compressionLevel);
    /* avoid const char * -> char * conversion warning :( */
    return (char*)(uptrval)bufferStart;
}


/* ================================================
 *  LZ4 Optimal parser (levels [LZ4HC_CLEVEL_OPT_MIN - LZ4HC_CLEVEL_MAX])
 * ===============================================*/
typedef struct {
    int price;
    int off;
    int mlen;
    int litlen;
} LZ4HC_optimal_t;

/* price in bytes */
LZ4_FORCE_INLINE int LZ4HC_literalsPrice(int const litlen)
{
    int price = litlen;
    assert(litlen >= 0);
    if (litlen >= (int)RUN_MASK)
        price += 1 + ((litlen-(int)RUN_MASK) / 255);
    return price;
}


/* requires mlen >= MINMATCH */
LZ4_FORCE_INLINE int LZ4HC_sequencePrice(int litlen, int mlen)
{
    int price = 1 + 2 ; /* token + 16-bit offset */
    assert(litlen >= 0);
    assert(mlen >= MINMATCH);

    price += LZ4HC_literalsPrice(litlen);

    if (mlen >= (int)(ML_MASK+MINMATCH))
        price += 1 + ((mlen-(int)(ML_MASK+MINMATCH)) / 255);

    return price;
}


typedef struct {
    int off;
    int len;
} LZ4HC_match_t;

LZ4_FORCE_INLINE LZ4HC_match_t
LZ4HC_FindLongerMatch(LZ4HC_CCtx_internal* const ctx,
                      const BYTE* ip, const BYTE* const iHighLimit,
                      int minLen, int nbSearches,
                      const dictCtx_directive dict,
                      const HCfavor_e favorDecSpeed)
{
    LZ4HC_match_t match = { 0 , 0 };
    const BYTE* matchPtr = NULL;
    /* note : LZ4HC_InsertAndGetWiderMatch() is able to modify the starting position of a match (*startpos),
     * but this won't be the case here, as we define iLowLimit==ip,
     * so LZ4HC_InsertAndGetWiderMatch() won't be allowed to search past ip */
    int matchLength = LZ4HC_InsertAndGetWiderMatch(ctx, ip, ip, iHighLimit, minLen, &matchPtr, &ip, nbSearches, 1 /*patternAnalysis*/, 1 /*chainSwap*/, dict, favorDecSpeed);
    if (matchLength <= minLen) return match;
    if (favorDecSpeed) {
        if ((matchLength>18) & (matchLength<=36)) matchLength=18;   /* favor shortcut */
    }
    match.len = matchLength;
    match.off = (int)(ip-matchPtr);
    return match;
}


static int LZ4HC_compress_optimal ( LZ4HC_CCtx_internal* ctx,
                                    const char* const source,
                                    char* dst,
                                    int* srcSizePtr,
                                    int dstCapacity,
                                    int const nbSearches,
                                    size_t sufficient_len,
                                    const limitedOutput_directive limit,
                                    int const fullUpdate,
                                    const dictCtx_directive dict,
                                    const HCfavor_e favorDecSpeed)
{
    int retval = 0;
#define TRAILING_LITERALS 3
#if defined(LZ4HC_HEAPMODE) && LZ4HC_HEAPMODE==1
    LZ4HC_optimal_t* const opt = (LZ4HC_optimal_t*)ALLOC(sizeof(LZ4HC_optimal_t) * (LZ4_OPT_NUM + TRAILING_LITERALS));
#else
    LZ4HC_optimal_t opt[LZ4_OPT_NUM + TRAILING_LITERALS];   /* ~64 KB, which is a bit large for stack... */
#endif

    const BYTE* ip = (const BYTE*) source;
    const BYTE* anchor = ip;
    const BYTE* const iend = ip + *srcSizePtr;
    const BYTE* const mflimit = iend - MFLIMIT;
    const BYTE* const matchlimit = iend - LASTLITERALS;
    BYTE* op = (BYTE*) dst;
    BYTE* opSaved = (BYTE*) dst;
    BYTE* oend = op + dstCapacity;
    int ovml = MINMATCH;  /* overflow - last sequence */
    const BYTE* ovref = NULL;

    /* init */
#if defined(LZ4HC_HEAPMODE) && LZ4HC_HEAPMODE==1
    if (opt == NULL) goto _return_label;
#endif
    DEBUGLOG(5, "LZ4HC_compress_optimal(dst=%p, dstCapa=%u)", dst, (unsigned)dstCapacity);
    *srcSizePtr = 0;
    if (limit == fillOutput) oend -= LASTLITERALS;   /* Hack for support LZ4 format restriction */
    if (sufficient_len >= LZ4_OPT_NUM) sufficient_len = LZ4_OPT_NUM-1;

    /* Main Loop */
    while (ip <= mflimit) {
         int const llen = (int)(ip - anchor);
         int best_mlen, best_off;
         int cur, last_match_pos = 0;

         LZ4HC_match_t const firstMatch = LZ4HC_FindLongerMatch(ctx, ip, matchlimit, MINMATCH-1, nbSearches, dict, favorDecSpeed);
         if (firstMatch.len==0) { ip++; continue; }

         if ((size_t)firstMatch.len > sufficient_len) {
             /* good enough solution : immediate encoding */
             int const firstML = firstMatch.len;
             const BYTE* const matchPos = ip - firstMatch.off;
             opSaved = op;
             if ( LZ4HC_encodeSequence(UPDATABLE(ip, op, anchor), firstML, matchPos, limit, oend) ) {  /* updates ip, op and anchor */
                 ovml = firstML;
                 ovref = matchPos;
                 goto _dest_overflow;
             }
             continue;
         }

         /* set prices for first positions (literals) */
         {   int rPos;
             for (rPos = 0 ; rPos < MINMATCH ; rPos++) {
                 int const cost = LZ4HC_literalsPrice(llen + rPos);
                 opt[rPos].mlen = 1;
                 opt[rPos].off = 0;
                 opt[rPos].litlen = llen + rPos;
                 opt[rPos].price = cost;
                 DEBUGLOG(7, "rPos:%3i => price:%3i (litlen=%i) -- initial setup",
                             rPos, cost, opt[rPos].litlen);
         }   }
         /* set prices using initial match */
         {   int mlen = MINMATCH;
             int const matchML = firstMatch.len;   /* necessarily < sufficient_len < LZ4_OPT_NUM */
             int const offset = firstMatch.off;
             assert(matchML < LZ4_OPT_NUM);
             for ( ; mlen <= matchML ; mlen++) {
                 int const cost = LZ4HC_sequencePrice(llen, mlen);
                 opt[mlen].mlen = mlen;
                 opt[mlen].off = offset;
                 opt[mlen].litlen = llen;
                 opt[mlen].price = cost;
                 DEBUGLOG(7, "rPos:%3i => price:%3i (matchlen=%i) -- initial setup",
                             mlen, cost, mlen);
         }   }
         last_match_pos = firstMatch.len;
         {   int addLit;
             for (addLit = 1; addLit <= TRAILING_LITERALS; addLit ++) {
                 opt[last_match_pos+addLit].mlen = 1; /* literal */
                 opt[last_match_pos+addLit].off = 0;
                 opt[last_match_pos+addLit].litlen = addLit;
                 opt[last_match_pos+addLit].price = opt[last_match_pos].price + LZ4HC_literalsPrice(addLit);
                 DEBUGLOG(7, "rPos:%3i => price:%3i (litlen=%i) -- initial setup",
                             last_match_pos+addLit, opt[last_match_pos+addLit].price, addLit);
         }   }

         /* check further positions */
         for (cur = 1; cur < last_match_pos; cur++) {
             const BYTE* const curPtr = ip + cur;
             LZ4HC_match_t newMatch;

             if (curPtr > mflimit) break;
             DEBUGLOG(7, "rPos:%u[%u] vs [%u]%u",
                     cur, opt[cur].price, opt[cur+1].price, cur+1);
             if (fullUpdate) {
                 /* not useful to search here if next position has same (or lower) cost */
                 if ( (opt[cur+1].price <= opt[cur].price)
                   /* in some cases, next position has same cost, but cost rises sharply after, so a small match would still be beneficial */
                   && (opt[cur+MINMATCH].price < opt[cur].price + 3/*min seq price*/) )
                     continue;
             } else {
                 /* not useful to search here if next position has same (or lower) cost */
                 if (opt[cur+1].price <= opt[cur].price) continue;
             }

             DEBUGLOG(7, "search at rPos:%u", cur);
             if (fullUpdate)
                 newMatch = LZ4HC_FindLongerMatch(ctx, curPtr, matchlimit, MINMATCH-1, nbSearches, dict, favorDecSpeed);
             else
                 /* only test matches of minimum length; slightly faster, but misses a few bytes */
                 newMatch = LZ4HC_FindLongerMatch(ctx, curPtr, matchlimit, last_match_pos - cur, nbSearches, dict, favorDecSpeed);
             if (!newMatch.len) continue;

             if ( ((size_t)newMatch.len > sufficient_len)
               || (newMatch.len + cur >= LZ4_OPT_NUM) ) {
                 /* immediate encoding */
                 best_mlen = newMatch.len;
                 best_off = newMatch.off;
                 last_match_pos = cur + 1;
                 goto encode;
             }

             /* before match : set price with literals at beginning */
             {   int const baseLitlen = opt[cur].litlen;
                 int litlen;
                 for (litlen = 1; litlen < MINMATCH; litlen++) {
                     int const price = opt[cur].price - LZ4HC_literalsPrice(baseLitlen) + LZ4HC_literalsPrice(baseLitlen+litlen);
                     int const pos = cur + litlen;
                     if (price < opt[pos].price) {
                         opt[pos].mlen = 1; /* literal */
                         opt[pos].off = 0;
                         opt[pos].litlen = baseLitlen+litlen;
                         opt[pos].price = price;
                         DEBUGLOG(7, "rPos:%3i => price:%3i (litlen=%i)",
                                     pos, price, opt[pos].litlen);
             }   }   }

             /* set prices using match at position = cur */
             {   int const matchML = newMatch.len;
                 int ml = MINMATCH;

                 assert(cur + newMatch.len < LZ4_OPT_NUM);
                 for ( ; ml <= matchML ; ml++) {
                     int const pos = cur + ml;
                     int const offset = newMatch.off;
                     int price;
                     int ll;
                     DEBUGLOG(7, "testing price rPos %i (last_match_pos=%i)",
                                 pos, last_match_pos);
                     if (opt[cur].mlen == 1) {
                         ll = opt[cur].litlen;
                         price = ((cur > ll) ? opt[cur - ll].price : 0)
                               + LZ4HC_sequencePrice(ll, ml);
                     } else {
                         ll = 0;
                         price = opt[cur].price + LZ4HC_sequencePrice(0, ml);
                     }

                    assert((U32)favorDecSpeed <= 1);
                     if (pos > last_match_pos+TRAILING_LITERALS
                      || price <= opt[pos].price - (int)favorDecSpeed) {
                         DEBUGLOG(7, "rPos:%3i => price:%3i (matchlen=%i)",
                                     pos, price, ml);
                         assert(pos < LZ4_OPT_NUM);
                         if ( (ml == matchML)  /* last pos of last match */
                           && (last_match_pos < pos) )
                             last_match_pos = pos;
                         opt[pos].mlen = ml;
                         opt[pos].off = offset;
                         opt[pos].litlen = ll;
                         opt[pos].price = price;
             }   }   }
             /* complete following positions with literals */
             {   int addLit;
                 for (addLit = 1; addLit <= TRAILING_LITERALS; addLit ++) {
                     opt[last_match_pos+addLit].mlen = 1; /* literal */
                     opt[last_match_pos+addLit].off = 0;
                     opt[last_match_pos+addLit].litlen = addLit;
                     opt[last_match_pos+addLit].price = opt[last_match_pos].price + LZ4HC_literalsPrice(addLit);
                     DEBUGLOG(7, "rPos:%3i => price:%3i (litlen=%i)", last_match_pos+addLit, opt[last_match_pos+addLit].price, addLit);
             }   }
         }  /* for (cur = 1; cur <= last_match_pos; cur++) */

         assert(last_match_pos < LZ4_OPT_NUM + TRAILING_LITERALS);
         best_mlen = opt[last_match_pos].mlen;
         best_off = opt[last_match_pos].off;
         cur = last_match_pos - best_mlen;

encode: /* cur, last_match_pos, best_mlen, best_off must be set */
         assert(cur < LZ4_OPT_NUM);
         assert(last_match_pos >= 1);  /* == 1 when only one candidate */
         DEBUGLOG(6, "reverse traversal, looking for shortest path (last_match_pos=%i)", last_match_pos);
         {   int candidate_pos = cur;
             int selected_matchLength = best_mlen;
             int selected_offset = best_off;
             while (1) {  /* from end to beginning */
                 int const next_matchLength = opt[candidate_pos].mlen;  /* can be 1, means literal */
                 int const next_offset = opt[candidate_pos].off;
                 DEBUGLOG(7, "pos %i: sequence length %i", candidate_pos, selected_matchLength);
                 opt[candidate_pos].mlen = selected_matchLength;
                 opt[candidate_pos].off = selected_offset;
                 selected_matchLength = next_matchLength;
                 selected_offset = next_offset;
                 if (next_matchLength > candidate_pos) break; /* last match elected, first match to encode */
                 assert(next_matchLength > 0);  /* can be 1, means literal */
                 candidate_pos -= next_matchLength;
         }   }

         /* encode all recorded sequences in order */
         {   int rPos = 0;  /* relative position (to ip) */
             while (rPos < last_match_pos) {
                 int const ml = opt[rPos].mlen;
                 int const offset = opt[rPos].off;
                 if (ml == 1) { ip++; rPos++; continue; }  /* literal; note: can end up with several literals, in which case, skip them */
                 rPos += ml;
                 assert(ml >= MINMATCH);
                 assert((offset >= 1) && (offset <= LZ4_DISTANCE_MAX));
                 opSaved = op;
                 if ( LZ4HC_encodeSequence(UPDATABLE(ip, op, anchor), ml, ip - offset, limit, oend) ) {  /* updates ip, op and anchor */
                     ovml = ml;
                     ovref = ip - offset;
                     goto _dest_overflow;
         }   }   }
     }  /* while (ip <= mflimit) */

_last_literals:
     /* Encode Last Literals */
     {   size_t lastRunSize = (size_t)(iend - anchor);  /* literals */
         size_t llAdd = (lastRunSize + 255 - RUN_MASK) / 255;
         size_t const totalSize = 1 + llAdd + lastRunSize;
         if (limit == fillOutput) oend += LASTLITERALS;  /* restore correct value */
         if (limit && (op + totalSize > oend)) {
             if (limit == limitedOutput) { /* Check output limit */
                retval = 0;
                goto _return_label;
             }
             /* adapt lastRunSize to fill 'dst' */
             lastRunSize  = (size_t)(oend - op) - 1 /*token*/;
             llAdd = (lastRunSize + 256 - RUN_MASK) / 256;
             lastRunSize -= llAdd;
         }
         DEBUGLOG(6, "Final literal run : %i literals", (int)lastRunSize);
         ip = anchor + lastRunSize; /* can be != iend if limit==fillOutput */

         if (lastRunSize >= RUN_MASK) {
             size_t accumulator = lastRunSize - RUN_MASK;
             *op++ = (RUN_MASK << ML_BITS);
             for(; accumulator >= 255 ; accumulator -= 255) *op++ = 255;
             *op++ = (BYTE) accumulator;
         } else {
             *op++ = (BYTE)(lastRunSize << ML_BITS);
         }
         LZ4_memcpy(op, anchor, lastRunSize);
         op += lastRunSize;
     }

     /* End */
     *srcSizePtr = (int) (((const char*)ip) - source);
     retval = (int) ((char*)op-dst);
     goto _return_label;

_dest_overflow:
if (limit == fillOutput) {
     /* Assumption : ip, anchor, ovml and ovref must be set correctly */
     size_t const ll = (size_t)(ip - anchor);
     size_t const ll_addbytes = (ll + 240) / 255;
     size_t const ll_totalCost = 1 + ll_addbytes + ll;
     BYTE* const maxLitPos = oend - 3; /* 2 for offset, 1 for token */
     DEBUGLOG(6, "Last sequence overflowing (only %i bytes remaining)", (int)(oend-1-opSaved));
     op = opSaved;  /* restore correct out pointer */
     if (op + ll_totalCost <= maxLitPos) {
         /* ll validated; now adjust match length */
         size_t const bytesLeftForMl = (size_t)(maxLitPos - (op+ll_totalCost));
         size_t const maxMlSize = MINMATCH + (ML_MASK-1) + (bytesLeftForMl * 255);
         assert(maxMlSize < INT_MAX); assert(ovml >= 0);
         if ((size_t)ovml > maxMlSize) ovml = (int)maxMlSize;
         if ((oend + LASTLITERALS) - (op + ll_totalCost + 2) - 1 + ovml >= MFLIMIT) {
             DEBUGLOG(6, "Space to end : %i + ml (%i)", (int)((oend + LASTLITERALS) - (op + ll_totalCost + 2) - 1), ovml);
             DEBUGLOG(6, "Before : ip = %p, anchor = %p", ip, anchor);
             LZ4HC_encodeSequence(UPDATABLE(ip, op, anchor), ovml, ovref, notLimited, oend);
             DEBUGLOG(6, "After : ip = %p, anchor = %p", ip, anchor);
     }   }
     goto _last_literals;
}
_return_label:
#if defined(LZ4HC_HEAPMODE) && LZ4HC_HEAPMODE==1
     FREEMEM(opt);
#endif
     return retval;
}
