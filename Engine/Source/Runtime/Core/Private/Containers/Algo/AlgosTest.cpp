// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "Math/UnrealMathUtility.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Misc/AutomationTest.h"
#include "Algo/AllOf.h"
#include "Algo/Compare.h"
#include "Algo/Contains.h"
#include "Algo/Copy.h"
#include "Algo/Heapify.h"
#include "Algo/HeapSort.h"
#include "Algo/Includes.h"
#include "Algo/IndexOf.h"
#include "Algo/IntroSort.h"
#include "Algo/IsHeap.h"
#include "Algo/IsSorted.h"
#include "Algo/LevenshteinDistance.h"
#include "Algo/Mismatch.h"
#include "Algo/NoneOf.h"
#include "Algo/RemoveIf.h"
#include "Algo/Sort.h"
#include "Algo/Transform.h"
#include "Templates/Greater.h"

namespace UE
{
namespace Impl
{

	struct FFixedTestRangeUnsigned
	{
		using SizeType = uint8;

		FFixedTestRangeUnsigned()
		{
			uint8 Count = 0;
			for (uint8& N : Numbers)
			{
				N = Count++;
			}
		}
		uint8 Num() const
		{
			return UE_ARRAY_COUNT(Numbers);
		}
		const uint8* GetData() const
		{
			return Numbers;
		}
		uint8 Numbers[255];
	};

} // namespace Impl
} // namespace UE

template<> struct TIsContiguousContainer<UE::Impl::FFixedTestRangeUnsigned>{ enum { Value = true }; };

class FAlgosTestBase : public FAutomationTestBase
{
public:
	using FAutomationTestBase::FAutomationTestBase;

	static constexpr int32 NumTestObjects = 32;

	struct FTestData
	{
		FTestData(FString&& InName, int32 InAge, bool bInRetired = false)
			: Name(MoveTemp(InName))
			, Age(InAge)
			, bRetired(bInRetired)
		{
		}

		friend bool operator==(const FTestData& A, const FTestData&B)
		{
			return A.Name == B.Name && A.Age == B.Age && A.bRetired == B.bRetired;
		}
		bool IsTeenager() const
		{
			return Age >= 13 && Age <= 19;
		}

		FString GetName() const
		{
			return Name;
		}

		FString Name;
		int32 Age;
		bool bRetired;
	};

	void Initialize()
	{
		for (int i = 0; i < NumTestObjects; ++i)
		{
			TestData.Add(i);
		}
		for (int i = 0; i < NumTestObjects; ++i)
		{
			TestData2.Add(FMath::Rand());
		}
	}

	void Cleanup()
	{
		TestData2.Empty();
		TestData.Empty();
	}

	void TestCopy()
	{
		TArray<int> TestArray;
		// empty array
		Algo::Copy(TestData, TestArray);
		check(TestArray == TestData);
		// existing data
		Algo::Copy(TestData2, TestArray);
		check(TestArray.Num() == NumTestObjects * 2);
		for (int i = 0; i < NumTestObjects; ++i)
		{
			check(TestArray[i] == TestData[i]);
		}
		for (int i = 0; i < NumTestObjects; ++i)
		{
			check(TestArray[i + NumTestObjects] == TestData2[i]);
		}
	}

	void TestCopyIf()
	{
		TArray<int> TestArray;
		// empty array
		Algo::CopyIf(TestData, TestArray, [](int i) { return (i % 2) == 0; });
		int j = 0;
		for (int i = 0; i < NumTestObjects; ++i)
		{
			if (TestData[i] % 2 == 0)
			{
				check(TestArray[j] == TestData[i]);
				++j;
			}
		}
		// existing data
		Algo::CopyIf(TestData2, TestArray, [](int i) { return (i % 2) == 0; });
		j = 0;
		for (int i = 0; i < NumTestObjects; ++i)
		{
			if (TestData[i] % 2 == 0)
			{
				check(TestArray[j] == TestData[i]);
				++j;
			}
		}
		for (int i = 0; i < NumTestObjects; ++i)
		{
			if (TestData2[i] % 2 == 0)
			{
				check(TestArray[j] == TestData2[i]);
				++j;
			}
		}
		check(j == TestArray.Num())
	}

	void TestTransform()
	{
		TArray<float> TestArray;

		// empty array
		{
			Algo::Transform(TestData, TestArray, [](int i) { return FMath::DegreesToRadians((float)i); });
			check(TestArray.Num() == NumTestObjects);
			for (int i = 0; i < TestArray.Num(); ++i)
			{
				check(TestArray[i] == FMath::DegreesToRadians((float)TestData[i]));
			}
		}

		// existing data
		{
			Algo::Transform(TestData2, TestArray, [](int i) { return FMath::DegreesToRadians((float)i); });
			check(TestArray.Num() == NumTestObjects * 2);
			for (int i = 0; i < NumTestObjects; ++i)
			{
				check(TestArray[i] == FMath::DegreesToRadians((float)TestData[i]));
			}
			for (int i = 0; i < NumTestObjects; ++i)
			{
				check(TestArray[i + NumTestObjects] == FMath::DegreesToRadians((float)TestData2[i]));
			}
		}

		// projection via member function pointer
		{
			TArray<FString> Strings = {
				TEXT("Hello"),
				TEXT("this"),
				TEXT("is"),
				TEXT("a"),
				TEXT("projection"),
				TEXT("test")
			};

			TArray<int32> Lengths;
			Algo::Transform(Strings, Lengths, &FString::Len);
			check(Lengths == TArray<int32>({ 5, 4, 2, 1, 10, 4 }));
		}

		// projection via data member pointer
		{
			TArray<FTestData> Data = {
				FTestData(TEXT("Alice"), 31),
				FTestData(TEXT("Bob"), 25),
				FTestData(TEXT("Charles"), 19),
				FTestData(TEXT("Donna"), 13)
			};

			TArray<int32> Ages;
			Algo::Transform(Data, Ages, &FTestData::Age);

			check(Ages == TArray<int32>({ 31, 25, 19, 13 }));
		}

		// projection across smart pointers
		{
			TArray<TUniquePtr<FTestData>> Data;
			Data.Add(MakeUnique<FTestData>(TEXT("Elsa"), 61));
			Data.Add(MakeUnique<FTestData>(TEXT("Fred"), 11));
			Data.Add(MakeUnique<FTestData>(TEXT("Georgina"), 34));
			Data.Add(MakeUnique<FTestData>(TEXT("Henry"), 54));
			Data.Add(MakeUnique<FTestData>(TEXT("Ichabod"), 87));

			TArray<FString> Names;
			Algo::Transform(Data, Names, &FTestData::Name);

			TArray<FString> ExpectedNames = { TEXT("Elsa"), TEXT("Fred"), TEXT("Georgina"), TEXT("Henry"), TEXT("Ichabod") };
			check(Names == ExpectedNames);
		}
	}

	void TestTransformIf()
	{
		TArray<float> TestArray;

		// empty array
		{
			Algo::TransformIf(TestData, TestArray, [](int i) { return (i % 2) == 0; }, [](int i) { return FMath::DegreesToRadians((float)i); });
			int j = 0;
			for (int i = 0; i < NumTestObjects; ++i)
			{
				if (TestData[i] % 2 == 0)
				{
					check(TestArray[j] == FMath::DegreesToRadians((float)TestData[i]));
					++j;
				}
			}
		}

		// existing data
		{
			Algo::TransformIf(TestData2, TestArray, [](int i) { return (i % 2) == 0; }, [](int i) { return FMath::DegreesToRadians((float)i); });
			int j = 0;
			for (int i = 0; i < NumTestObjects; ++i)
			{
				if (TestData[i] % 2 == 0)
				{
					check(TestArray[j] == FMath::DegreesToRadians((float)TestData[i]));
					++j;
				}
			}
			for (int i = 0; i < NumTestObjects; ++i)
			{
				if (TestData2[i] % 2 == 0)
				{
					check(TestArray[j] == FMath::DegreesToRadians((float)TestData2[i]));
					++j;
				}
			}
			check(j == TestArray.Num());
		}

		TArray<TUniquePtr<FTestData>> Data;
		Data.Add(MakeUnique<FTestData>(TEXT("Jeff"), 15, false));
		Data.Add(MakeUnique<FTestData>(TEXT("Katrina"), 77, true));
		Data.Add(MakeUnique<FTestData>(TEXT("Lenny"), 29, false));
		Data.Add(MakeUnique<FTestData>(TEXT("Michelle"), 13, false));
		Data.Add(MakeUnique<FTestData>(TEXT("Nico"), 65, true));

		// projection and transform via data member pointer
		{
			TArray<FString> NamesOfRetired;
			Algo::TransformIf(Data, NamesOfRetired, &FTestData::bRetired, &FTestData::Name);
			TArray<FString> ExpectedNamesOfRetired = { TEXT("Katrina"), TEXT("Nico") };
			check(NamesOfRetired == ExpectedNamesOfRetired);
		}

		// projection and transform via member function pointer
		{
			TArray<FString> NamesOfTeenagers;
			Algo::TransformIf(Data, NamesOfTeenagers, &FTestData::IsTeenager, &FTestData::GetName);
			TArray<FString> ExpectedNamesOfTeenagers = { TEXT("Jeff"), TEXT("Michelle") };
			check(NamesOfTeenagers == ExpectedNamesOfTeenagers);
		}
	}

	void TestBinarySearch()
	{
		// Verify static array case
		int StaticArray[] = { 2,4,6,6,6,8 };

		check(Algo::BinarySearch(StaticArray, 6) == 2);
		check(Algo::BinarySearch(StaticArray, 5) == INDEX_NONE);
		check(Algo::BinarySearchBy(StaticArray, 4, FIdentityFunctor()) == 1);

		check(Algo::LowerBound(StaticArray, 6) == 2);
		check(Algo::LowerBound(StaticArray, 5) == 2);
		check(Algo::UpperBound(StaticArray, 6) == 5);
		check(Algo::LowerBound(StaticArray, 7) == 5);
		check(Algo::LowerBound(StaticArray, 9) == 6);
		check(Algo::LowerBoundBy(StaticArray, 6, FIdentityFunctor()) == 2);
		check(Algo::UpperBoundBy(StaticArray, 6, FIdentityFunctor()) == 5);

		// Dynamic array case
		TArray<int32> IntArray = { 2,2,4,4,6,6,6,8,8 };

		check(Algo::BinarySearch(IntArray, 6) == 4);
		check(Algo::BinarySearch(IntArray, 5) == INDEX_NONE);
		check(Algo::BinarySearchBy(IntArray, 4, FIdentityFunctor()) == 2);

		check(Algo::LowerBound(IntArray, 2) == 0);
		check(Algo::UpperBound(IntArray, 2) == 2);
		check(Algo::LowerBound(IntArray, 6) == 4);
		check(Algo::UpperBound(IntArray, 6) == 7);
		check(Algo::LowerBound(IntArray, 5) == 4);
		check(Algo::UpperBound(IntArray, 5) == 4);
		check(Algo::LowerBound(IntArray, 7) == 7);
		check(Algo::LowerBound(IntArray, 9) == 9);
		check(Algo::LowerBoundBy(IntArray, 6, FIdentityFunctor()) == 4);
		check(Algo::UpperBoundBy(IntArray, 6, FIdentityFunctor()) == 7);
	}

	void TestIndexOf()
	{
		TArray<FTestData> Data = {
			FTestData(TEXT("Alice"), 31),
			FTestData(TEXT("Bob"), 25),
			FTestData(TEXT("Charles"), 19),
			FTestData(TEXT("Donna"), 13)
		};

		int FixedArray[] = { 2,4,6,6,6,8 };
		check(Algo::IndexOf(FixedArray, 2) == 0);
		check(Algo::IndexOf(FixedArray, 6) == 2);
		check(Algo::IndexOf(FixedArray, 8) == 5);
		check(Algo::IndexOf(FixedArray, 0) == INDEX_NONE);

		check(Algo::IndexOf(Data, FTestData(TEXT("Alice"), 31)) == 0);
		check(Algo::IndexOf(Data, FTestData(TEXT("Alice"), 32)) == INDEX_NONE);

		check(Algo::IndexOfBy(Data, TEXT("Donna"), &FTestData::Name) == 3);
		check(Algo::IndexOfBy(Data, 19, &FTestData::Age) == 2);
		check(Algo::IndexOfBy(Data, 0, &FTestData::Age) == INDEX_NONE);

		auto GetAge = [](const FTestData& In) { return In.Age; };
		check(Algo::IndexOfBy(Data, 19, GetAge) == 2);
		check(Algo::IndexOfBy(Data, 0, GetAge) == INDEX_NONE);

		check(Algo::IndexOfByPredicate(Data, [](const FTestData& In) { return In.Age < 25; }) == 2);
		check(Algo::IndexOfByPredicate(Data, [](const FTestData& In) { return In.Age > 19; }) == 0);
		check(Algo::IndexOfByPredicate(Data, [](const FTestData& In) { return In.Age > 31; }) == INDEX_NONE);

		static const uint8 InvalidIndex = (uint8)-1;
		UE::Impl::FFixedTestRangeUnsigned TestRange;
		check(Algo::IndexOf(TestRange, 25) == 25);
		check(Algo::IndexOf(TestRange, 254) == 254);
		check(Algo::IndexOf(TestRange, 255) == InvalidIndex);
		check(Algo::IndexOf(TestRange, 1024) == InvalidIndex);
	}

	void TestHeapify()
	{
		TArray<int> TestArray = TestData2;
		Algo::Heapify(TestArray);

		check(Algo::IsHeap(TestArray));
	}

	void TestHeapSort()
	{
		TArray<int> TestArray = TestData2;
		Algo::HeapSort(TestArray);

		check(Algo::IsHeap(TestArray));

		check(Algo::IsSorted(TestArray));
	}

	void TestIntroSort()
	{
		TArray<int> TestArray = TestData2;
		Algo::IntroSort(TestArray);

		check(Algo::IsSorted(TestArray));
	}

	void TestSort()
	{
		// regular Sort
		TArray<int> TestArray = TestData2;
		Algo::Sort(TestArray);

		check(Algo::IsSorted(TestArray));

		// Sort with predicate
		TestArray = TestData2;

		TGreater<> Predicate;
		Algo::Sort(TestArray, Predicate);

		check(Algo::IsSorted(TestArray, Predicate));

		// SortBy
		TestArray = TestData2;

		auto Projection = [](int Val) -> int
		{
			return Val % 1000; // will sort using the last 3 digits only
		};

		Algo::SortBy(TestArray, Projection);

		check(Algo::IsSortedBy(TestArray, Projection));

		// SortBy with predicate
		TestArray = TestData2;

		Algo::SortBy(TestArray, Projection, Predicate);

		check(Algo::IsSortedBy(TestArray, Projection, Predicate));
	}

	void TestEditDistance()
	{
		struct FEditDistanceTestData
		{
			const TCHAR* A;
			const TCHAR* B;
			ESearchCase::Type SearchCase;
			int32 ExpectedResultDistance;
		};

		const FEditDistanceTestData EditDistanceTests[] =
		{
			//Empty tests
			{ TEXT(""), TEXT("Saturday"), ESearchCase::CaseSensitive, 8 },
			{ TEXT(""), TEXT("Saturday"), ESearchCase::IgnoreCase, 8 },
			{ TEXT("Saturday"), TEXT(""), ESearchCase::CaseSensitive, 8 },
			{ TEXT("Saturday"), TEXT(""), ESearchCase::IgnoreCase, 8 },
			//One letter tests
			{ TEXT("a"), TEXT("a"), ESearchCase::CaseSensitive, 0 },
			{ TEXT("a"), TEXT("b"), ESearchCase::CaseSensitive, 1 },
			//Equal tests
			{ TEXT("Saturday"), TEXT("Saturday"), ESearchCase::CaseSensitive, 0 },
			{ TEXT("Saturday"), TEXT("Saturday"), ESearchCase::IgnoreCase, 0 },
			//Simple casing test
			{ TEXT("Saturday"), TEXT("saturday"), ESearchCase::CaseSensitive, 1 },
			{ TEXT("Saturday"), TEXT("saturday"), ESearchCase::IgnoreCase, 0 },
			{ TEXT("saturday"), TEXT("Saturday"), ESearchCase::CaseSensitive, 1 },
			{ TEXT("saturday"), TEXT("Saturday"), ESearchCase::IgnoreCase, 0 },
			{ TEXT("SaturdaY"), TEXT("saturday"), ESearchCase::CaseSensitive, 2 },
			{ TEXT("SaturdaY"), TEXT("saturday"), ESearchCase::IgnoreCase, 0 },
			{ TEXT("saturdaY"), TEXT("Saturday"), ESearchCase::CaseSensitive, 2 },
			{ TEXT("saturdaY"), TEXT("Saturday"), ESearchCase::IgnoreCase, 0 },
			{ TEXT("SATURDAY"), TEXT("saturday"), ESearchCase::CaseSensitive, 8 },
			{ TEXT("SATURDAY"), TEXT("saturday"), ESearchCase::IgnoreCase, 0 },
			//First char diff
			{ TEXT("Saturday"), TEXT("baturday"), ESearchCase::CaseSensitive, 1 },
			{ TEXT("Saturday"), TEXT("baturday"), ESearchCase::IgnoreCase, 1 },
			//Last char diff
			{ TEXT("Saturday"), TEXT("Saturdai"), ESearchCase::CaseSensitive, 1 },
			{ TEXT("Saturday"), TEXT("Saturdai"), ESearchCase::IgnoreCase, 1 },
			//Middle char diff
			{ TEXT("Satyrday"), TEXT("Saturday"), ESearchCase::CaseSensitive, 1 },
			{ TEXT("Satyrday"), TEXT("Saturday"), ESearchCase::IgnoreCase, 1 },
			//Real cases
			{ TEXT("Copy_Body"), TEXT("Body"), ESearchCase::CaseSensitive, 5 },
			{ TEXT("Copy_Body"), TEXT("Body"), ESearchCase::IgnoreCase, 5 },
			{ TEXT("copy_Body"), TEXT("Paste_Body"), ESearchCase::CaseSensitive, 5 },
			{ TEXT("copy_Body"), TEXT("Paste_Body"), ESearchCase::IgnoreCase, 5 },
			{ TEXT("legs"), TEXT("Legs_1"), ESearchCase::CaseSensitive, 3 },
			{ TEXT("legs"), TEXT("Legs_1"), ESearchCase::IgnoreCase, 2 },
			{ TEXT("arms"), TEXT("Arms"), ESearchCase::CaseSensitive, 1 },
			{ TEXT("arms"), TEXT("Arms"), ESearchCase::IgnoreCase, 0 },
			{ TEXT("Saturday"), TEXT("Sunday"), ESearchCase::CaseSensitive, 3 },
			{ TEXT("Saturday"), TEXT("Sunday"), ESearchCase::IgnoreCase, 3 },
			{ TEXT("Saturday"), TEXT("suNday"), ESearchCase::CaseSensitive, 4 },
			{ TEXT("Saturday"), TEXT("suNday"), ESearchCase::IgnoreCase, 3 },
			{ TEXT("Saturday"), TEXT("sUnday"), ESearchCase::CaseSensitive, 5 },
			{ TEXT("Saturday"), TEXT("sUnday"), ESearchCase::IgnoreCase, 3 },
		};

		for (const FEditDistanceTestData& Test : EditDistanceTests)
		{
			RunEditDistanceTest(Test.A, Test.B, Test.SearchCase, Test.ExpectedResultDistance);
		}
	}

	void RunEditDistanceTest(const FString& A, const FString& B, const ESearchCase::Type SearchCase, const int32 ExpectedResultDistance)
	{
		// Run test
		int32 ResultDistance = MAX_int32;
		if (SearchCase == ESearchCase::IgnoreCase)
		{
			ResultDistance = Algo::LevenshteinDistance(A.ToLower(), B.ToLower());
		}
		else
		{
			ResultDistance = Algo::LevenshteinDistance(A, B);
		}

		if (ResultDistance != ExpectedResultDistance)
		{
			FString SearchCaseStr = SearchCase == ESearchCase::CaseSensitive ? TEXT("CaseSensitive") : TEXT("IgnoreCase");
			AddError(FString::Printf(TEXT("Algo::EditDistance return the wrong distance between 2 string (A '%s', B '%s', case '%s', result '%d', expected '%d')."), *A, *B, *SearchCaseStr, ResultDistance, ExpectedResultDistance));
		}
	}

	void TestEditDistanceArray()
	{
		struct FEditDistanceArrayTestData
		{
			const TCHAR* ArrayDescriptionA;
			const TCHAR* ArrayDescriptionB;
			TArray<int32> A;
			TArray<int32> B;
			int32 ExpectedResultDistance;
		};

		const FEditDistanceArrayTestData EditDistanceArrayTests[] =
		{
			//Identical array
			{ TEXT("{1, 2, 3, 4}"), TEXT("{1, 2, 3, 4}"), {1, 2, 3, 4}, {1, 2, 3, 4}, 0 },
			//1 difference
			{ TEXT("{1, 2, 3, 4}"), TEXT("{1, 2, 3, 10}"), {1, 2, 3, 4}, {1, 2, 3, 10}, 1 },
			//1 character less
			{ TEXT("{1, 2, 3, 4}"), TEXT("{1, 2, 3}"), {1, 2, 3, 4}, {1, 2, 3}, 1 },
			//1 character more
			{ TEXT("{1, 2, 3, 4}"), TEXT("{1, 2, 3, 4, 5}"), {1, 2, 3, 4}, {1, 2, 3, 4, 5}, 1 },
			//2 character more
			{ TEXT("{1, 2, 3, 4}"), TEXT("{1, 2, 3, 4, 5, 6}"), {1, 2, 3, 4}, {1, 2, 3, 4, 5, 6}, 2 },
			//B string empty
			{ TEXT("{1, 2, 3, 4}"), TEXT("{}"), {1, 2, 3, 4}, {}, 4 },
		};

		for (const FEditDistanceArrayTestData& Test : EditDistanceArrayTests)
		{
			RunEditDistanceTestArray(Test.ArrayDescriptionA, Test.ArrayDescriptionB, Test.A, Test.B, Test.ExpectedResultDistance);
		}
	}

	void RunEditDistanceTestArray(const FString& ArrayDescriptionA, const FString& ArrayDescriptionB, const TArray<int32>& A, const TArray<int32>& B, const int32 ExpectedResultDistance)
	{
		// Run test
		int32 ResultDistance = Algo::LevenshteinDistance(A, B);

		if (ResultDistance != ExpectedResultDistance)
		{
			AddError(FString::Printf(TEXT("Algo::EditDistance return the wrong distance between 2 array (A '%s', B '%s', result '%d', expected '%d')."), *ArrayDescriptionA, *ArrayDescriptionB, ResultDistance, ExpectedResultDistance));
		}
	}

	void TestIncludes()
	{
		// Fixed arrays with elements of fundamental types - test Algo::Includes overloads
		{
			constexpr int32 FixedArrayA[] = { 1, 2, 3, 4, 5 };
			constexpr int32 FixedArrayB[] = { 1, 3, 4 };
			constexpr int32 FixedArrayC[] = { 5, 4, 3, 2, 1 };
			constexpr int32 FixedArrayD[] = { 4, 3, 1 };
			const TArrayView<int32> Empty;

			// Test case 1: A contains A as a subsequence.
			check(Algo::Includes(FixedArrayA, FixedArrayA));
			// Test case 2: A contains B as a subsequence.
			check(Algo::Includes(FixedArrayA, FixedArrayB));
			// Test case 3: A contains Empty as a subsequence, because an empty set is always considered a subset of a non-empty set.
			check(Algo::Includes(FixedArrayA, Empty));
			// Test case 4: Empty contains Empty as a subsequence, because an empty set is always considered a subset of an empty set.
			check(Algo::Includes(Empty, Empty));
			// Test case 5: Empty doesn't contain A as a subsequence, because an empty set can't contain any elements of a non-empty set.
			check(!Algo::Includes(Empty, FixedArrayA));
			// Test case 6: B doesn't contain A as a subsequence, because A contains elements B doesn't have.
			check(!Algo::Includes(FixedArrayB, FixedArrayA));
			// Test case 7: C doesn't contain B as a subsequence, because C isn't ordered according to the default comparison predicate (operator<).
			check(!Algo::Includes(FixedArrayC, FixedArrayB));
			// Test case 8: A doesn't contain D as a subsequence, because D isn't ordered according to the default comparison predicate (operator<).
			check(!Algo::Includes(FixedArrayA, FixedArrayD));

			// Test case 9: C contains C as a subsequence with TGreater<> as the comparison predicate.
			check(Algo::Includes(FixedArrayC, FixedArrayC, TGreater<>()));
			// Test case 10: C contains D as a subsequence with TGreater<> as the comparison predicate.
			check(Algo::Includes(FixedArrayC, FixedArrayD, TGreater<>()));
			// Test case 11: C contains Empty as a subsequence, because an empty set is always considered a subset of a non-empty set.
			check(Algo::Includes(FixedArrayC, Empty, TGreater<>()));
			// Test case 12: Empty contains Empty as a subsequence, because an empty set is always considered a subset of an empty set.
			check(Algo::Includes(Empty, Empty, TGreater<>()));
			// Test case 13: Empty doesn't contain C as a subsequence, because an empty set can't contain any elements of a non-empty set.
			check(!Algo::Includes(Empty, FixedArrayC, TGreater<>()));
			// Test case 14: D doesn't contain C as a subsequence, because C contains elements D doesn't have.
			check(!Algo::Includes(FixedArrayD, FixedArrayC, TGreater<>()));
			// Test case 15: A doesn't contain D as a subsequence, because A isn't ordered according to the custom comparison predicate (TGreater<>()).
			check(!Algo::Includes(FixedArrayA, FixedArrayD, TGreater<>()));
			// Test case 16: C doesn't contain B as a subsequence, because B isn't ordered according to the custom comparison predicate (TGreater<>()).
			check(!Algo::Includes(FixedArrayC, FixedArrayB, TGreater<>()));
		}

		// Dynamic arrays with elements of compound types - test Algo::IncludesBy
		{
			TArray<FTestData> DynamicArrayA =
			{
				FTestData(TEXT("1"), 1),
				FTestData(TEXT("2"), 2),
				FTestData(TEXT("3"), 3),
				FTestData(TEXT("4"), 4),
				FTestData(TEXT("5"), 5)
			};
			TArray<FTestData> DynamicArrayB =
			{
				FTestData(TEXT("1"), 1),
				FTestData(TEXT("3"), 3),
				FTestData(TEXT("4"), 4)
			};
			TArray<FTestData> DynamicArrayC =
			{
				FTestData(TEXT("5"), 5),
				FTestData(TEXT("4"), 4),
				FTestData(TEXT("3"), 3),
				FTestData(TEXT("2"), 2),
				FTestData(TEXT("1"), 1)
			};
			TArray<FTestData> DynamicArrayD =
			{
				FTestData(TEXT("4"), 4),
				FTestData(TEXT("3"), 3),
				FTestData(TEXT("1"), 1)
			};
			const TArrayView<FTestData> Empty;

			// Test case 1: A contains A as a subsequence.
			check(Algo::IncludesBy(DynamicArrayA, DynamicArrayA, &FTestData::Name));
			// Test case 2: A contains B as a subsequence.
			check(Algo::IncludesBy(DynamicArrayA, DynamicArrayB, &FTestData::Name));
			// Test case 3: A contains Empty as a subsequence, because an empty set is always considered a subset of a non-empty set.
			check(Algo::IncludesBy(DynamicArrayA, Empty, &FTestData::Name));
			// Test case 4: Empty contains Empty as a subsequence, because an empty set is always considered a subset of an empty set.
			check(Algo::IncludesBy(Empty, Empty, &FTestData::Name));
			// Test case 5: Empty doesn't contain A as a subsequence, because an empty set can't contain any elements of a non-empty set.
			check(!Algo::IncludesBy(Empty, DynamicArrayA, &FTestData::Name));
			// Test case 6: B doesn't contain A as a subsequence, because A contains elements B doesn't have.
			check(!Algo::IncludesBy(DynamicArrayB, DynamicArrayA, &FTestData::Name));
			// Test case 7: C doesn't contain B as a subsequence, because C isn't ordered according to the default comparison predicate (operator<).
			check(!Algo::IncludesBy(DynamicArrayC, DynamicArrayB, &FTestData::Name));
			// Test case 8: A doesn't contain D as a subsequence, because D isn't ordered according to the default comparison predicate (operator<).
			check(!Algo::IncludesBy(DynamicArrayA, DynamicArrayD, &FTestData::Name));

			// Test case 9: C contains C as a subsequence with TGreater<> as the comparison predicate.
			check(Algo::IncludesBy(DynamicArrayC, DynamicArrayC, &FTestData::Name, TGreater<>()));
			// Test case 10: C contains D as a subsequence with TGreater<> as the comparison predicate.
			check(Algo::IncludesBy(DynamicArrayC, DynamicArrayD, &FTestData::Name, TGreater<>()));
			// Test case 11: C contains Empty as a subsequence, because an empty set is always considered a subset of a non-empty set.
			check(Algo::IncludesBy(DynamicArrayC, Empty, &FTestData::Name, TGreater<>()));
			// Test case 12: Empty contains Empty as a subsequence, because an empty set is always considered a subset of an empty set.
			check(Algo::IncludesBy(Empty, Empty, &FTestData::Name, TGreater<>()));
			// Test case 13: Empty doesn't contain C as a subsequence, because an empty set can't contain any elements of a non-empty set.
			check(!Algo::IncludesBy(Empty, DynamicArrayC, &FTestData::Name, TGreater<>()));
			// Test case 14: D doesn't contain C as a subsequence, because C contains elements D doesn't have.
			check(!Algo::IncludesBy(DynamicArrayD, DynamicArrayC, &FTestData::Name, TGreater<>()));
			// Test case 15: A doesn't contain D as a subsequence, because A isn't ordered according to the custom comparison predicate (TGreater<>()).
			check(!Algo::IncludesBy(DynamicArrayA, DynamicArrayD, &FTestData::Name, TGreater<>()));
			// Test case 16: C doesn't contain B as a subsequence, because B isn't ordered according to the custom comparison predicate (TGreater<>()).
			check(!Algo::IncludesBy(DynamicArrayC, DynamicArrayB, &FTestData::Name, TGreater<>()));
		}
	}

	void TestMismatch()
	{
		{
			TArray<int32> Empty;
			TArray<int32> DataA = { 1, 2, 3, 4, 5, 6 };
			TArray<int32> DataB = { 1, 2, 3, 7, 8, 9 };
			TArray<int32> DataC = { 1, 2, 3, 4, 5, 6, 7, 8, 9 };

			// Test empty ranges
			check(Algo::Mismatch(Empty, Empty) == 0);
			check(Algo::Mismatch(Empty, DataA) == 0);
			check(Algo::Mismatch(DataA, Empty) == 0);

			// Test common initial sequences
			check(Algo::Mismatch(DataA, DataB) == 3);
			check(Algo::Mismatch(DataB, DataA) == 3);

			// Test equal sequences
			check(Algo::Mismatch(DataA, DataA) == 6);
			check(Algo::Mismatch(DataB, DataB) == 6);
			check(Algo::Mismatch(DataC, DataC) == 9);

			// Test subsequences
			check(Algo::Mismatch(DataA, DataC) == 6);
			check(Algo::Mismatch(DataC, DataA) == 6);
		}

		{
			auto CompareCaseInsensitive = [](TCHAR Lhs, TCHAR Rhs)
			{
				return FChar::ToUpper(Lhs) == FChar::ToUpper(Rhs);
			};

			FString Empty;
			FString DataA = "HeLlO wOrLd";
			FString DataB = "HELLO GOODBYE";
			FString DataC = "hello";

			// Test empty ranges with custom equality
			check(Algo::Mismatch(Empty, Empty, CompareCaseInsensitive) == 0);
			check(Algo::Mismatch(Empty, DataA, CompareCaseInsensitive) == 0);
			check(Algo::Mismatch(DataA, Empty, CompareCaseInsensitive) == 0);

			// Test common initial sequences
			check(Algo::Mismatch(DataA, DataB, CompareCaseInsensitive) == 6);
			check(Algo::Mismatch(DataB, DataA, CompareCaseInsensitive) == 6);

			// Test equal sequences
			check(Algo::Mismatch(DataA, DataA, CompareCaseInsensitive) == 11);
			check(Algo::Mismatch(DataB, DataB, CompareCaseInsensitive) == 13);
			check(Algo::Mismatch(DataC, DataC, CompareCaseInsensitive) == 5);

			// Test subsequences
			check(Algo::Mismatch(DataA, DataC, CompareCaseInsensitive) == 5);
			check(Algo::Mismatch(DataC, DataA, CompareCaseInsensitive) == 5);
			check(Algo::Mismatch(DataB, DataC, CompareCaseInsensitive) == 5);
			check(Algo::Mismatch(DataC, DataB, CompareCaseInsensitive) == 5);
		}

		{
			auto Square = [](int32 Val)
			{
				return Val * Val;
			};

			TArray<int32> Empty;
			TArray<int32> DataA = { 1, 2, 3, 4, 5, 6 };
			TArray<int32> DataB = { -1, -2, -3, -7, -8, -9 };
			TArray<int32> DataC = { 1, -2, 3, -4, 5, -6, 7, -8, 9 };

			// Test empty ranges with projection
			check(Algo::MismatchBy(Empty, Empty, Square) == 0);
			check(Algo::MismatchBy(Empty, DataA, Square) == 0);
			check(Algo::MismatchBy(DataA, Empty, Square) == 0);

			// Test common initial sequences with projection
			check(Algo::MismatchBy(DataA, DataB, Square) == 3);
			check(Algo::MismatchBy(DataB, DataA, Square) == 3);

			// Test equal sequences with projection
			check(Algo::MismatchBy(DataA, DataA, Square) == 6);
			check(Algo::MismatchBy(DataB, DataB, Square) == 6);
			check(Algo::MismatchBy(DataC, DataC, Square) == 9);

			// Test subsequences with projection
			check(Algo::MismatchBy(DataA, DataC, Square) == 6);
			check(Algo::MismatchBy(DataC, DataA, Square) == 6);
		}

		{
			struct FStringWrapper
			{
				FString Str;
			};

			auto CompareCaseInsensitive = [](const FString& Lhs, const FString& Rhs)
			{
				return Lhs.Equals(Rhs, ESearchCase::IgnoreCase);
			};

			TArray<FStringWrapper> Empty;
			TArray<FStringWrapper> DataA = { { TEXT("Class") }, { TEXT("Struct") }, { TEXT("Enum") }, { TEXT("Float") }, { TEXT("Int") }, { TEXT("Char") } };
			TArray<FStringWrapper> DataB = { { TEXT("class") }, { TEXT("struct") }, { TEXT("enum") }, { TEXT("public") }, { TEXT("protected") }, { TEXT("private") } };
			TArray<FStringWrapper> DataC = { { TEXT("CLASS") }, { TEXT("STRUCT") }, { TEXT("ENUM") }, { TEXT("FLOAT") }, { TEXT("INT") }, { TEXT("CHAR") }, { TEXT("PUBLIC") }, { TEXT("PROTECTED") }, { TEXT("PRIVATE") } };

			// Test empty ranges with projection and custom equality
			check(Algo::MismatchBy(Empty, Empty, &FStringWrapper::Str, CompareCaseInsensitive) == 0);
			check(Algo::MismatchBy(Empty, DataA, &FStringWrapper::Str, CompareCaseInsensitive) == 0);
			check(Algo::MismatchBy(DataA, Empty, &FStringWrapper::Str, CompareCaseInsensitive) == 0);

			// Test common initial sequences with projection and custom equality
			check(Algo::MismatchBy(DataA, DataB, &FStringWrapper::Str, CompareCaseInsensitive) == 3);
			check(Algo::MismatchBy(DataB, DataA, &FStringWrapper::Str, CompareCaseInsensitive) == 3);

			// Test equal sequences with projection and custom equality
			check(Algo::MismatchBy(DataA, DataA, &FStringWrapper::Str, CompareCaseInsensitive) == 6);
			check(Algo::MismatchBy(DataB, DataB, &FStringWrapper::Str, CompareCaseInsensitive) == 6);
			check(Algo::MismatchBy(DataC, DataC, &FStringWrapper::Str, CompareCaseInsensitive) == 9);

			// Test subsequences with projection and custom equality
			check(Algo::MismatchBy(DataA, DataC, &FStringWrapper::Str, CompareCaseInsensitive) == 6);
			check(Algo::MismatchBy(DataC, DataA, &FStringWrapper::Str, CompareCaseInsensitive) == 6);
		}
	}

	void TestRemoveIf()
	{
		struct FStringWrapper
		{
			FString Str;
			bool bMovedFrom = false;
			mutable bool bPredicateInvoked = false;

			FStringWrapper(FString&& InStr)
				: Str(MoveTemp(InStr))
			{
			}
			~FStringWrapper() = default;

			// These shouldn't be called
			FStringWrapper(FStringWrapper&&) = delete;
			FStringWrapper(const FStringWrapper&) = delete;
			bool operator=(const FStringWrapper&) = delete;

			FStringWrapper& operator=(FStringWrapper&& Rhs)
			{
				checkf(!Rhs.bMovedFrom, TEXT("Algo::RemoveIf - attempting to move a moved-from object"));
				Str = MoveTemp(Rhs.Str);
				bMovedFrom = false;
				Rhs.bMovedFrom = true;
				return *this;
			}
		};

		auto Test = [](std::initializer_list<const ANSICHAR*> Input, std::initializer_list<const ANSICHAR*> ExpectedResult)
		{
			{
				TArray<FStringWrapper> TestArray;
				for (const ANSICHAR* InputStr : Input)
				{
					TestArray.Emplace(InputStr);
				}

				int32 NumCalls = 0;
				auto ShouldRemove = [&NumCalls](const FStringWrapper& Wrapper)
				{
					++NumCalls;
					checkf(!Wrapper.bPredicateInvoked, TEXT("Algo::RemoveIf - running predicate on the same object multiple times"));
					Wrapper.bPredicateInvoked = true;
					return Wrapper.Str.StartsWith(TEXT("Remove-"));
				};

				int32 Result = Algo::RemoveIf(TestArray, ShouldRemove);

				checkf(NumCalls == Input.size(), TEXT("Algo::RemoveIf - expected predicate to be run %d times, got %d"), (int32)Input.size(), NumCalls);
				checkf(Result == ExpectedResult.size(), TEXT("Algo::RemoveIf - expected %d elements to be kept, got %d"), (int32)ExpectedResult.size(), Result);
				checkf(Algo::AllOf(TestArray, &FStringWrapper::bPredicateInvoked), TEXT("Algo::RemoveIf - expected predicate to be run over all elements but found one that was not"));

				TestArray.RemoveAt(Result, TestArray.Num() - Result, EAllowShrinking::No);

				checkf(Algo::NoneOf(TestArray, &FStringWrapper::bMovedFrom), TEXT("Algo::RemoveIf - found non-removed element in a moved-from state"));
				checkf(Algo::AllOf(ExpectedResult, [&TestArray](const ANSICHAR* ExpectedStr){ return Algo::ContainsBy(TestArray, ExpectedStr, &FStringWrapper::Str); }), TEXT("Algo::RemoveIf - actual result does not match expected result"));
			}

			{
				TArray<FStringWrapper> TestArray;
				for (const ANSICHAR* InputStr : Input)
				{
					TestArray.Emplace(InputStr);
				}

				int32 NumCalls = 0;
				auto ShouldRemove = [&NumCalls](const FStringWrapper& Wrapper)
				{
					++NumCalls;
					checkf(!Wrapper.bPredicateInvoked, TEXT("Algo::StableRemoveIf - running predicate on the same object multiple times"));
					Wrapper.bPredicateInvoked = true;
					return Wrapper.Str.StartsWith(TEXT("Remove-"));
				};

				int32 Result = Algo::StableRemoveIf(TestArray, ShouldRemove);

				checkf(NumCalls == Input.size(), TEXT("Algo::StableRemoveIf - expected predicate to be run %d times, got %d"), (int32)Input.size(), NumCalls);
				checkf(Result == ExpectedResult.size(), TEXT("Algo::StableRemoveIf - expected %d elements to be kept, got %d"), (int32)ExpectedResult.size(), Result);
				checkf(Algo::AllOf(TestArray, &FStringWrapper::bPredicateInvoked), TEXT("Algo::StableRemoveIf - expected predicate to be run over all elements but found one that was not"));

				TestArray.RemoveAt(Result, TestArray.Num() - Result, EAllowShrinking::No);

				checkf(Algo::NoneOf(TestArray, &FStringWrapper::bMovedFrom), TEXT("Algo::StableRemoveIf - found non-removed element in a moved-from state"));
				checkf(Algo::Compare(TestArray, ExpectedResult, [](const FStringWrapper& Lhs, const ANSICHAR* Rhs){ return Lhs.Str == Rhs; }), TEXT("Algo::StableRemoveIf - actual result does not match expected result"));
			}
		};

		// Test empty range
		Test({}, {});

		// Test none removed
		Test({ "1", "2", "3", "4" }, { "1", "2", "3", "4" });

		// Test all removed
		Test({ "Remove-1", "Remove-2", "Remove-3", "Remove-4" }, {});

		// Test only the first removed
		Test({ "Remove-1", "2", "3", "4" }, { "2", "3", "4" });

		// Test all but the first removed
		Test({ "1", "Remove-2", "Remove-3", "Remove-4" }, { "1" });

		// Test only the last removed
		Test({ "1", "2", "3", "Remove-4" }, { "1", "2", "3" });

		// Test all but the last removed
		Test({ "Remove-1", "Remove-2", "Remove-3", "4" }, { "4" });

		// Test a mixture
		Test({ "Remove-1", "2", "3", "4", "Remove-5", "Remove-6", "7", "Remove-8", "9" }, { "2", "3", "4", "7", "9" });
	}

private:
	TArray<int> TestData;
	TArray<int> TestData2;
};

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FAlgosTest, FAlgosTestBase, "System.Core.Misc.Algos", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)

bool FAlgosTest::RunTest(const FString& Parameters)
{
	Initialize();
	TestCopy();
	TestCopyIf();
	TestTransform();
	TestTransformIf();
	TestBinarySearch();
	TestIndexOf();
	TestHeapify();
	TestHeapSort();
	TestIntroSort();
	TestSort();
	TestEditDistance();
	TestEditDistanceArray();
	TestIncludes();
	TestMismatch();
	TestRemoveIf();
	Cleanup();

	return true;
}
