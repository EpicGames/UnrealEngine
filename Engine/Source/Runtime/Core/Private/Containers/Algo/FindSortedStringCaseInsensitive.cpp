// Copyright Epic Games, Inc. All Rights Reserved.

#include "Algo/FindSortedStringCaseInsensitive.h"

#include "Misc/CString.h"
#include "Misc/Char.h"

namespace Algo
{
	/**
	 * Finds a string in an array of sorted strings, by case-insensitive search, by using binary subdivision of the array.
	 *
	 * @param  Str          The string to look for.
	 * @param  SortedArray  The array of strings to search.  The strings must be sorted lexicographically, case-insensitively.
	 * @param  ArrayCount   The number of strings in the array.
	 *
	 * @return The index of the found string in the array, or -1 if the string was not found.
	 */
	int32 FindSortedStringCaseInsensitive(const TCHAR* Str, const TCHAR* const* SortedArray, int32 ArrayCount)
	{
		// The current index of the character of Str that we're using for our binary search
		int32 CharIndex = 0;

		// Begin/end iterators to the range of strings that within SortedArray have matched our string up to CharIndex
		auto SubArray    = SortedArray;
		auto SubArrayEnd = SubArray + ArrayCount;

		for (;;)
		{
			// If the subarray is empty, then we've failed to find a match
			if (SubArray == SubArrayEnd)
			{
				return -1;
			}

			// If the subarray is only 1 element, we can use Stricmp to finish the job
			if (SubArray + 1 == SubArrayEnd)
			{
				return (FCString::Stricmp(*SubArray + CharIndex, Str + CharIndex) == 0) ? UE_PTRDIFF_TO_INT32(SubArray - SortedArray) : -1;
			}

			// Get the next char to binary search for
			TCHAR Ch = FChar::ToLower(Str[CharIndex]);

			// If there are no more characters to search for then we're done
			if (Ch == TEXT('\0'))
			{
				// If the subarray's string is also at the end then we've found a match,
				// otherwise we've only found a prefix, which is not a match
				return ((*SubArray)[CharIndex] == TEXT('\0')) ? UE_PTRDIFF_TO_INT32(SubArray - SortedArray) : -1;
			}

			// Now we try to find the equal_range of the character
			for (;;)
			{
				// If the subarray is empty, then we've failed to find a match
				if (SubArray == SubArrayEnd)
				{
					return -1;
				}

				// Find midpoint of the subarray and its char
				auto Midpoint = SubArray + (SubArrayEnd - SubArray) / 2;
				TCHAR ArrCh = FChar::ToLower((*Midpoint)[CharIndex]);

				if (ArrCh < Ch)
				{
					// The midpoint char is before the string char, so narrow the subrange to after the midpoint
					SubArray = Midpoint + 1;
				}
				else if (ArrCh > Ch)
				{
					// The midpoint char is after the string char, so narrow the subrange to before the midpoint
					SubArrayEnd = Midpoint;
				}
				else
				{
					// The midpoint char is the string char, so binary search [SubArray, Midpoint) to find the lower bound
					auto LowerBoundEnd = Midpoint;
					while (SubArray != LowerBoundEnd)
					{
						auto LowerBoundMid = SubArray + (LowerBoundEnd - SubArray) / 2;

						TCHAR LowerBoundCh = FChar::ToLower((*LowerBoundMid)[CharIndex]);
						if (LowerBoundCh < Ch)
						{
							SubArray = LowerBoundMid + 1;
						}
						else
						{
							LowerBoundEnd = LowerBoundMid;
						}
					}

					// Then binary search [Midpoint, SubArrayEnd) to find the upper bound
					auto UpperBound = Midpoint + 1;
					while (UpperBound != SubArrayEnd)
					{
						auto UpperBoundMid = UpperBound + (SubArrayEnd - UpperBound) / 2;

						TCHAR UpperBoundCh = FChar::ToLower((*UpperBoundMid)[CharIndex]);
						if (Ch >= UpperBoundCh)
						{
							UpperBound = UpperBoundMid + 1;
						}
						else
						{
							SubArrayEnd = UpperBoundMid;
						}
					}

					break;
				}
			}

			++CharIndex;
		}
	}
}
