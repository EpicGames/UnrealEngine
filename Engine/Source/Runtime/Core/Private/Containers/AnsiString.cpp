// Copyright Epic Games, Inc. All Rights Reserved.

#include "Containers/AnsiString.h"
#include "Containers/StringIncludes.cpp.inl"

#define UE_STRING_CLASS             FAnsiString
#define UE_STRING_CHARTYPE          ANSICHAR
#define UE_STRING_CHARTYPE_IS_TCHAR 0
	#include "Containers/String.cpp.inl"
#undef UE_STRING_CHARTYPE_IS_TCHAR
#undef UE_STRING_CHARTYPE
#undef UE_STRING_CLASS
