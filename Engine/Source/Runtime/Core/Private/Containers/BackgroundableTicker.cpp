// Copyright Epic Games, Inc. All Rights Reserved.

#include "Containers/BackgroundableTicker.h"

#include "Containers/Ticker.h"
#include "Delegates/DelegateInstancesImpl.h"
#include "Misc/CoreDelegates.h"
#include "Stats/Stats.h"

FTSBackgroundableTicker& FTSBackgroundableTicker::GetCoreTicker()
{
	static FTSBackgroundableTicker Singleton;
	return Singleton;
}

FTSBackgroundableTicker::FTSBackgroundableTicker()
{
	CoreTickerHandle = FTSTicker::GetCoreTicker().AddTicker(TEXT("FBackgroundableTicker"), 0.0f, [this](float DeltaTime) -> bool
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FBackgroundableTicker_ForegroundTick);
			if (bWasBackgrounded)
			{
				// When returning to the foreground, ensure we do not report enormous delta time values coming from the foreground ticker.
				bWasBackgrounded = false;
				DeltaTime = FMath::Clamp(DeltaTime, 0.0f, 1.0f / 60.0f);
			}
			Tick(DeltaTime);
			return true;
		});
	BackgroundTickerHandle = FCoreDelegates::MobileBackgroundTickDelegate.AddLambda([this](float DeltaTime)
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FBackgroundableTicker_BackgroundTick);
			bWasBackgrounded = true;
			Tick(DeltaTime);
		});
}

FTSBackgroundableTicker::~FTSBackgroundableTicker()
{
	FTSTicker::RemoveTicker(CoreTickerHandle);
	FCoreDelegates::MobileBackgroundTickDelegate.Remove(BackgroundTickerHandle);
}
