// Copyright Epic Games, Inc. All Rights Reserved.

#include "Containers/BitArray.h"

#include "Math/NumericLimits.h"

void FBitArrayMemory::MemmoveBitsWordOrder(uint32* StartDest, int32 DestOffset, const uint32* StartSource, int32 SourceOffset, uint32 NumBits)
{
	// Normalize Inputs
	check(NumBits >= 0);
	if (NumBits <= 0)
	{
		return;
	}
	ModularizeWordOffset(StartDest, DestOffset);
	ModularizeWordOffset(StartSource, SourceOffset);

	// If the Dest words are aligned with the Source Words, use the aligned version of MemmoveBits which requires fewer masking instructions.
	// This also allows us to reduce complexity by not needing to handle the aligned case in the rest of this function.
	const int32 DeltaOffset = DestOffset - SourceOffset;
	if (DeltaOffset == 0)
	{
		MemmoveBitsWordOrderAlignedInternal(StartDest, StartSource, DestOffset, NumBits);
		return;
	}

	// Work out shifts for which bits of each source word are in each of the two dest words it overlaps 
	const uint32 FirstBitInLowerDestWord = (DeltaOffset >= 0) ? DeltaOffset : DeltaOffset + NumBitsPerDWORD;
	const uint32 UpShiftToLowerDestWord = FirstBitInLowerDestWord; // The shift up distance to align with the dest positions of the lower dest word is equal to the position of the first bit in that word. We copy it to this variable name to make it easier to understand why we're using it as the shift
	const uint32 DownShiftToUpperDestWord = NumBitsPerDWORD - FirstBitInLowerDestWord; // The shift down distance to align with the upper dest word is the additive inverse of the shift-up distance

	// Calculate Starting and Ending DestMasks
	const uint32 EndDestOffset = (DestOffset + NumBits - 1) % NumBitsPerDWORD + 1;
	const uint32 StartDestMask = DestOffset == 0 ? MAX_uint32 : (MAX_uint32 << DestOffset);
	const uint32 EndDestMask = MAX_uint32 >> (NumBitsPerDWORD - EndDestOffset);

	// Calculate Final Pointers
	uint32* const FinalDest = StartDest + (DestOffset + NumBits - 1) / NumBitsPerDWORD;
	const uint32* const FinalSource = StartSource + (SourceOffset + NumBits - 1) / NumBitsPerDWORD;

	// If the end word is the start word, use special handling to apply both the start mask and the end mask
	if (StartDest == FinalDest)
	{
		uint32 CombinedDestMask = StartDestMask & EndDestMask;
		// Read the bits from the lower and upper source word of the dest word, if they are in the source range
		// Note we are not allowed to try reading from any word that does not overlap the source range because it could pointing to inaccessible memory.
		if (SourceOffset < DestOffset)
		{
			// The beginning of the first source word is past the beginning of the dest word, so it is the upper source word of the dest word
			// It must therefore also be the last source word.
			// Shift it up to the offset of the dest word.
			const uint32 DestBitsFromUpperSourceWord = *(StartSource) << UpShiftToLowerDestWord;
			*StartDest = (*StartDest & ~CombinedDestMask) | (DestBitsFromUpperSourceWord & CombinedDestMask);
		}
		else if (StartSource == FinalSource)
		{
			// The beginning of the first source word is before the beginning of the dest word, so it is the lower source word of the dest word
			// But the write range ends at or before the end of the source word, so we only have a single source word
			// Shift it down to the offset of the dest word
			const uint32 DestBitsFromLowerSourceWord = ((*StartSource) >> DownShiftToUpperDestWord);
			*StartDest = (*StartDest & ~CombinedDestMask) | (DestBitsFromLowerSourceWord & CombinedDestMask);
		}
		else
		{
			// The beginning of the first source word is before the beginning of the dest word, so it is the lower source word of the dest word
			// The write range ends after the end of the source word, so we also have an upper source word - FinalSource is the upper source word of the destword
			// Shift StartSource down and FinalSource up
			const uint32 DestBitsFromLowerSourceWord = ((*StartSource) >> DownShiftToUpperDestWord);
			const uint32 DestBitsFromUpperSourceWord = *(FinalSource) << UpShiftToLowerDestWord;
			*StartDest = (*StartDest & ~CombinedDestMask) | ((DestBitsFromLowerSourceWord | DestBitsFromUpperSourceWord) & CombinedDestMask);
		}
		return;
	}

	// If the Dest range and Source range overlap, we need to dest the words of dest in the same order as the direction from deststart to sourcestart, so that words are read from the destrange before being overwritten
	// If Dest is less than Source, this is handled by forward enumeration, which is the default since it is better for cache coherency
	// If Source is less than Dest AND they overlap, we need to enumerate backwards
	const bool bRequiresBackward = (StartSource < StartDest) & (StartDest <= FinalSource);
	if (!bRequiresBackward)
	{
		uint32* CurrentDest = StartDest;
		const uint32* CurrentSource = StartSource;
		uint32 DestBitsFromLowerSourceWord;
		uint32 DestBitsFromUpperSourceWord;
		uint32 NextDestBitsFromLowerSourceWord;

		// Write the first Dest word, handling whether the first source word is the lower source word or the upper source word and handling StartDestMask
		if (SourceOffset < DestOffset)
		{
			// The beginning of the first source word is past the beginning of the dest word, so it is the upper source word of the dest word, and we don't have bits from the lower source word
			// Note we are not allowed to try reading from any word that does not overlap the source range because it could pointing to inaccessible memory, so we don't try to read the bits from CurrentSource-1
			DestBitsFromUpperSourceWord = *(CurrentSource) << UpShiftToLowerDestWord;
			NextDestBitsFromLowerSourceWord = *(CurrentSource) >> DownShiftToUpperDestWord; // Must read this before writing CurrentDest to handle overlap
			*CurrentDest = (*CurrentDest & ~(StartDestMask)) | (DestBitsFromUpperSourceWord & StartDestMask);
		}
		else
		{
			// The first source word is the lower source word of the first dest word
			DestBitsFromLowerSourceWord = *(CurrentSource) >> DownShiftToUpperDestWord;
			++CurrentSource;
			DestBitsFromUpperSourceWord = *(CurrentSource) << UpShiftToLowerDestWord;
			NextDestBitsFromLowerSourceWord = *(CurrentSource) >> DownShiftToUpperDestWord; // Must read this before writing CurrentDest to handle overlap
			*CurrentDest = (*CurrentDest & ~(StartDestMask)) | ((DestBitsFromLowerSourceWord | DestBitsFromUpperSourceWord) & StartDestMask);
		}

		// Establish the loop invariant: we have written all dest before CurrentDest, CurrentSource is the lower source word of CurrentDest, and DestBitsFromLowerSourceWord has been read
		// Note that reading all bits from CurrentSource before writing any dest words it touches is necessary if the source range and dest range overlap and the source words are within one word of their dest words
		++CurrentDest;
		// CurrentSource remains unchanged; it is the lower source word of the new CurrentDest
		DestBitsFromLowerSourceWord = NextDestBitsFromLowerSourceWord;

		while (CurrentDest != FinalDest)
		{
			++CurrentSource;
			DestBitsFromUpperSourceWord = *(CurrentSource) << UpShiftToLowerDestWord;
			NextDestBitsFromLowerSourceWord = *(CurrentSource) >> DownShiftToUpperDestWord; // Must read this before writing CurrentDest to handle overlap
			*CurrentDest = DestBitsFromLowerSourceWord | DestBitsFromUpperSourceWord;
			++CurrentDest;
			DestBitsFromLowerSourceWord = NextDestBitsFromLowerSourceWord;
		}

		// Write the final Dest word, handling whether the last source word is the lower source word or the upper source word and handling endmask
		if (EndDestOffset <= FirstBitInLowerDestWord)
		{
			// The last dest word ends before the bits touched by its upper source word, so the current lower source word is the last source word
			// Note we are not allowed to try reading from any word that does not overlap the source range because it could pointing to inaccessible memory, so we don't try to read the bits from CurrentSource+1
			DestBitsFromUpperSourceWord = 0;
		}
		else
		{
			// The last source word is the upper source word of the last dest word; increment one last time and read the UpperSourceWord bits
			++CurrentSource;
			DestBitsFromUpperSourceWord = *(CurrentSource) << UpShiftToLowerDestWord;
		}
		*CurrentDest = ((DestBitsFromLowerSourceWord | DestBitsFromUpperSourceWord) & EndDestMask) | (*CurrentDest & ~(EndDestMask));
	}
	else
	{
		uint32* CurrentDest = FinalDest;
		const uint32* CurrentSource = FinalSource;
		uint32 DestBitsFromLowerSourceWord;
		uint32 DestBitsFromUpperSourceWord;
		uint32 NextDestBitsFromUpperSourceWord;

		// Write the final Dest word, handling whether the last source word is the lower source word or the upper source word and handling endmask
		if (EndDestOffset <= FirstBitInLowerDestWord)
		{
			// The last dest word ends before the bits touched by its upper source word, so the final source word is the lower source word for the final dest word, and we don't have any bits from the upper source word
			// Note we are not allowed to try reading from any word that does not overlap the source range because it could pointing to inaccessible memory, so we don't try to read the bits from CurrentSource+1
			DestBitsFromLowerSourceWord = *(CurrentSource) >> DownShiftToUpperDestWord;
			NextDestBitsFromUpperSourceWord = *(CurrentSource) << UpShiftToLowerDestWord; // Must read this before writing CurrentDest to handle overlap
			*CurrentDest = (DestBitsFromLowerSourceWord & EndDestMask) | (*CurrentDest & ~(EndDestMask));
		}
		else
		{
			// The last source word is the upper source word of the last dest word
			DestBitsFromUpperSourceWord = *(CurrentSource) << UpShiftToLowerDestWord;
			--CurrentSource;
			DestBitsFromLowerSourceWord = *(CurrentSource) >> DownShiftToUpperDestWord;
			NextDestBitsFromUpperSourceWord = *(CurrentSource) << UpShiftToLowerDestWord; // Must read this before writing CurrentDest to handle overlap
			*CurrentDest = ((DestBitsFromLowerSourceWord | DestBitsFromUpperSourceWord) & EndDestMask) | (*CurrentDest & ~(EndDestMask));
		}

		// Establish the loop invariant: we have written all dest before CurrentDest, CurrentSource is the lower source word of CurrentDest, and DestBitsFromLowerSourceWord has been read
		// Note that reading all bits from CurrentSource before writing any dest words it touches is necessary if the source range and dest range overlap and the source words are within one word of their dest words
		--CurrentDest;
		// CurrentSource remains unchanged; it is the upper source word of the new CurrentDest
		DestBitsFromUpperSourceWord = NextDestBitsFromUpperSourceWord;

		while (CurrentDest != StartDest)
		{
			--CurrentSource;
			DestBitsFromLowerSourceWord = *(CurrentSource) >> DownShiftToUpperDestWord;
			NextDestBitsFromUpperSourceWord = *(CurrentSource) << UpShiftToLowerDestWord; // Must read this before writing CurrentDest to handle overlap
			*CurrentDest = DestBitsFromLowerSourceWord | DestBitsFromUpperSourceWord;
			--CurrentDest;
			DestBitsFromUpperSourceWord = NextDestBitsFromUpperSourceWord;
		}

		// Write the first Dest word, handling whether the first source word is the lower source word or the upper source word and handling StartDestMask
		if (SourceOffset < DestOffset)
		{
			// The beginning of the first source word is past the beginning of the dest word, so it is the upper source word of the dest word, and we've already read its bits above
			// Note we are not allowed to try reading from any word that does not overlap the source range because it could pointing to inaccessible memory, so we don't try to read the bits from CurrentSource-1
			DestBitsFromLowerSourceWord = 0;
		}
		else
		{
			// The first source word is the lower source word of the first dest word; decrement one last time and read the LowerSourceWord bits
			--CurrentSource;
			DestBitsFromLowerSourceWord = *(CurrentSource) >> DownShiftToUpperDestWord;
		}
		*CurrentDest = (*CurrentDest & ~(StartDestMask)) | ((DestBitsFromLowerSourceWord | DestBitsFromUpperSourceWord) & StartDestMask);
	}
}

void FBitArrayMemory::MemmoveBitsWordOrderAlignedInternal(uint32*const StartDest, const uint32*const StartSource, int32 StartOffset, uint32 NumBits)
{
	// Contracts guaranteed by caller:
	// NumBits > 0
	// 0 <= StartOffset < NumBitsPerDWORD

	// Calculate Starting and EndingMasks
	const int32 EndOffset = (StartOffset + NumBits - 1) % NumBitsPerDWORD + 1;
	const uint32 StartMask = MAX_uint32 << StartOffset;
	const uint32 EndMask = MAX_uint32 >> (NumBitsPerDWORD - EndOffset);

	// Calculate Start and Final Pointers
	const uint32 OffsetToLastWord = (StartOffset + NumBits - 1) / NumBitsPerDWORD;
	uint32* const FinalDest = StartDest + OffsetToLastWord;
	const uint32* const FinalSource = StartSource + OffsetToLastWord;

	// If the end word is the start word, use special handling to apply both the start mask and the end mask
	if (OffsetToLastWord == 0)
	{
		uint32 CombinedMask = StartMask & EndMask;
		*StartDest = (*StartDest & ~CombinedMask) | (*StartSource & CombinedMask);
		return;
	}

	// If the Dest range and Source range overlap, we need to dest the words of dest in the same order as the direction from deststart to sourcestart, so that words are read from the destrange before being overwritten
	// If Dest is less than Source, this is handled by forward iteration, which is the default since it is better for cache coherency
	// If Source is less than Dest AND they overlap, we need to iterate backwards
	const bool bRequiresBackward = (StartSource < StartDest) & (StartDest <= FinalSource);
	if (!bRequiresBackward)
	{
		*StartDest = (*StartDest & ~StartMask) | (*StartSource & StartMask);
		uint32* CurrentDest = StartDest + 1;
		const uint32* CurrentSource = StartSource + 1;
		while (CurrentDest < FinalDest)
		{
			*CurrentDest++ = *CurrentSource++;
		}
		*FinalDest = (*FinalSource & EndMask) | (*FinalDest & ~EndMask);
	}
	else
	{
		*FinalDest = (*FinalSource & EndMask) | (*FinalDest & ~EndMask);
		uint32* CurrentDest = FinalDest - 1;
		const uint32* CurrentSource = FinalSource - 1;
		while (CurrentDest > StartDest)
		{
			*CurrentDest-- = *CurrentSource--;
		}
		*StartDest = (*StartDest & ~StartMask) | (*StartSource & StartMask);
	}
}

void FBitArrayMemory::ModularizeWordOffset(uint32 const*& Data, int32& Offset)
{
	if ((Offset < 0) | (NumBitsPerDWORD <= Offset))
	{
		int32 NumWords = Offset / NumBitsPerDWORD;
		Data += NumWords;
		Offset -= NumWords * NumBitsPerDWORD;
		if (Offset < 0)
		{
			--Data;
			Offset += NumBitsPerDWORD;
		}
	}
}
