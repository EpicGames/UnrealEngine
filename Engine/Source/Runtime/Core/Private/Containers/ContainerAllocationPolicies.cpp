// Copyright Epic Games, Inc. All Rights Reserved.

#include "Containers/ContainerAllocationPolicies.h"
#include "CoreGlobals.h"

FORCENOINLINE void UE::Core::Private::OnInvalidAlignedHeapAllocatorNum(int32 NewNum, SIZE_T NumBytesPerElement)
{
	UE_LOG(LogCore, Fatal, TEXT("Trying to resize TAlignedHeapAllocator to an invalid size of %d with element size %" SIZE_T_FMT), NewNum, NumBytesPerElement);
	for (;;);
}

FORCENOINLINE void UE::Core::Private::OnInvalidSizedHeapAllocatorNum(int32 IndexSize, int64 NewNum, SIZE_T NumBytesPerElement)
{
	UE_LOG(LogCore, Fatal, TEXT("Trying to resize TSizedHeapAllocator<%d> to an invalid size of %" INT64_FMT " with element size %" SIZE_T_FMT), IndexSize, NewNum, NumBytesPerElement);
	for (;;);
}

#if !UE_MERGED_MODULES && !defined(UE_HEADER_UNITS)
template void TSizedHeapAllocator<32, FMemory>::ForAnyElementType::ResizeAllocation(SizeType CurrentNum, SizeType NewMax, SIZE_T NumBytesPerElement);
template void TSizedHeapAllocator<32, FMemory>::ForAnyElementType::ResizeAllocation(SizeType CurrentNum, SizeType NewMax, SIZE_T NumBytesPerElement, uint32 AlignmentOfElement);
template void TSizedHeapAllocator<64, FMemory>::ForAnyElementType::ResizeAllocation(SizeType CurrentNum, SizeType NewMax, SIZE_T NumBytesPerElement);
template void TSizedHeapAllocator<64, FMemory>::ForAnyElementType::ResizeAllocation(SizeType CurrentNum, SizeType NewMax, SIZE_T NumBytesPerElement, uint32 AlignmentOfElement);
#endif