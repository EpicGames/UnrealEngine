// Copyright Epic Games, Inc. All Rights Reserved.

#include "Containers/ContainerHelpers.h"
#include "CoreGlobals.h"

void UE::Core::Private::OnInvalidArrayNum(unsigned long long NewNum)
{
	UE_LOGF(LogCore, Fatal, "Trying to resize TArray to an invalid size of %llu", NewNum);
	for (;;);
}

void UE::Core::Private::OnInvalidSetNum(unsigned long long NewNum)
{
	UE_LOGF(LogCore, Fatal, "Trying to resize TSet to an invalid size of %llu", NewNum);
	for (;;);
}
