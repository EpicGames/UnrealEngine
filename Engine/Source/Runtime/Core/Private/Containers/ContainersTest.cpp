// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "Concepts/EqualityComparable.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/SparseSet.h"
#include "Containers/CompactSet.h"
#include "Containers/SortedMap.h"
#include "Containers/SortedSet.h"
#include "Math/RandomStream.h"
#include "Misc/AssertionMacros.h"
#include "Misc/AutomationTest.h"
#include "Stats/StatsMisc.h"
#include "Templates/Tuple.h"
#include "Templates/UniquePtr.h"
#include "Containers/HashTable.h"

#if WITH_DEV_AUTOMATION_TESTS

#define MAX_TEST_OBJECTS      65
#define MAX_TEST_OBJECTS_STEP 1
#define RANDOM_SEED 12345

namespace
{
	struct FSetResolver
	{
		template<typename InElementType>
		using TSetType = TSet<InElementType>;

		template<typename InKey, typename InValueType>
		using TMapType = TMap<InKey, InValueType>;
	};

	struct FSparseSetResolver
	{
		template<typename InElementType>
		using TSetType = TSparseSet<InElementType>;

		template<typename InKey, typename InValueType>
		using TMapType = TSparseMap<InKey, InValueType>;
	};

	struct FCompactSetResolver
	{
		template<typename InElementType>
		using TSetType = TCompactSet<InElementType>;

		template<typename InKey, typename InValueType>
		using TMapType = TCompactMap<InKey, InValueType>;
	};

	struct FContainerTestStats
	{
		int32  NextId;
		int32  ConstructedIDs[MAX_TEST_OBJECTS];
		int32* End;

		FContainerTestStats()
		{
			Reset();
		}

		void Reset()
		{
			NextId = 1;
			End    = ConstructedIDs;
		}

		int32 Num() const
		{
			return UE_PTRDIFF_TO_INT32(End - ConstructedIDs);
		}

		int32 Add()
		{
			// Ensure we're not constructing too many objects
			check(Num() < MAX_TEST_OBJECTS);

			// Store ID in array
			return *End++ = NextId++;
		}

		void Remove(int32 ObjId)
		{
			for (int32* It = ConstructedIDs; It != End; ++It)
			{
				if (*It != ObjId)
					continue;

				// Remove this from the list
				--End;
				for (; It != End; ++It)
				{
					*It = *(It + 1);
				}
				return;
			}

			// We didn't find an entry for this - an invalid destructor call?
			check(false);
		}
	} ContainerTestStats;

	struct FContainerTestValueType
	{
		FContainerTestValueType()
			: FContainerTestValueType(TEXT("<default value>"))
		{
		}

		explicit FContainerTestValueType(const TCHAR* InStr)
			: Str(InStr)
			, Id(ContainerTestStats.Add())
		{
		}
		FContainerTestValueType(FContainerTestValueType&& Other)
			: Str(Other.Str)
			, Id(ContainerTestStats.Add())
		{
			Other.Str = nullptr;
			Other.bMovedFrom = true;
		}

		FContainerTestValueType(const FContainerTestValueType& Other)
			: Str(Other.Str)
			, Id(ContainerTestStats.Add())
			, bMovedFrom(Other.bMovedFrom)
		{
		}

		FContainerTestValueType& operator=(FContainerTestValueType&& Other)
		{
			Str = Other.Str;
			bMovedFrom = Other.bMovedFrom;

			Other.Str = nullptr;
			Other.bMovedFrom = true;

			return *this;
		}

		FContainerTestValueType& operator=(const FContainerTestValueType& Other)
		{
			Str = Other.Str;
			bMovedFrom = Other.bMovedFrom;

			return *this;
		}

		~FContainerTestValueType()
		{
			ContainerTestStats.Remove(Id);
		}

		const TCHAR* Str;
		int32 Id;
		bool bMovedFrom = false;

		friend bool operator==(const FContainerTestValueType& Lhs, const FContainerTestValueType& Rhs)
		{
			return Lhs.bMovedFrom == Rhs.bMovedFrom && Lhs.Id   == Rhs.Id && !FCString::Strcmp(Lhs.Str, Rhs.Str);
		}

		friend bool operator!=(const FContainerTestValueType& Lhs, const FContainerTestValueType& Rhs)
		{
			return !(Lhs == Rhs);
		}
	};

	template <typename Container>
	void CheckContainerElements(Container& Cont)
	{
		auto It  = Cont.CreateIterator();
		auto CIt = Cont.CreateConstIterator();
		for (auto& E : Cont)
		{
			check(*It  == E);
			check(*CIt == E);

			FSetElementId Id = It.GetId();
			FSetElementId CId = It.GetId();
			check(Cont.IsValidId(Id));
			check(Cont.IsValidId(CId));
			check(Cont.Get(Id) == E);
			check(Cont.Get(CId) == E);

			++It;
			++CIt;
		}
	}

	template <typename Container>
	void CheckContainerSelfEquality(Container& Cont)
	{
		if constexpr (TModels_V<CEqualityComparable, Container>)
		{
			check(Cont == Cont);
		}
	}

	template <typename Container>
	void CheckContainerNum(Container& Cont)
	{
		int32 Count = 0;
		for (auto It = Cont.CreateIterator(); It; ++It)
		{
			++Count;
		}

		int32 CCount = 0;
		for (auto It = Cont.CreateConstIterator(); It; ++It)
		{
			++CCount;
		}

		int32 RCount = 0;
		for (auto& It : Cont)
		{
			++RCount;
		}

		check(Count  == Cont.Num());
		check(CCount == Cont.Num());
		check(RCount == Cont.Num());
	}

	template <typename Container>
	void CheckContainerEnds(Container& Cont)
	{
		auto Iter  = Cont.CreateIterator();
		auto CIter = Cont.CreateConstIterator();

		for (int32 Num = Cont.Num(); Num; --Num)
		{
			++Iter;
			++CIter;
		}

		check(!Iter);
		check(!CIter);
	}

	template <typename Container>
	void CheckContainerSorted(Container& Cont)
	{
		using ElementType = typename Container::ElementType;

		const ElementType* PrevElement = nullptr;
		for (auto Iter = Cont.CreateConstIterator(); Iter; ++Iter)
		{
			const ElementType* CurrElement = &*Iter;
			check(!PrevElement || *PrevElement < *CurrElement);
			PrevElement = CurrElement;
		}
	}

	template <typename KeyType>
	struct TestKeyGenerator
	{
		static KeyType GetKey(int32 Input)
		{
			return (KeyType)Input;
		}
	};

	template <>
	struct TestKeyGenerator<FName>
	{
		static FName GetKey(int32 Input)
		{
			// Don't use _foo as we want to test the slower compare path
			return FName(*FString::Printf(TEXT("TestName%d"), Input));
		}
	};

	template <>
	struct TestKeyGenerator<FString>
	{
		static FString GetKey(int32 Input)
		{
			return FString::Printf(TEXT("TestString%d"), Input);
		}
	};

	template <typename Ti, typename Tj>
	struct TestKeyGenerator<TPair<Ti, Tj>>
	{
		static TPair<Ti, Tj> GetKey(int32 Input)
		{
			return TPair<Ti, Tj>(
				TestKeyGenerator<Ti>::GetKey(Input),
			    TestKeyGenerator<Tj>::GetKey(Input + 100));
		}
	};

	template <typename ArrayType>
	void RunPadToNumTests(const typename ArrayType::ElementType& PaddingValue1, const typename ArrayType::ElementType& PaddingValue2)
	{
		using ElementType = typename ArrayType::ElementType;

		ArrayType Array;

		// Check padding an empty array
		{
			auto Result = Array.PadToNumDefaulted(0);
			check(Result == 0);
			check(Array.IsEmpty());
		}
		{
			auto Result = Array.PadToNum(0, PaddingValue1);
			check(Result == 0);
			check(Array.IsEmpty());
		}

		// Check padding with a value
		{
			auto Result = Array.PadToNumDefaulted(5);
			check(Result == 5);
			check(Array.Num() == 5);
			check(Array == (ArrayType{ ElementType{}, ElementType{}, ElementType{}, ElementType{}, ElementType{} }));
		}

		// Check underpadding
		{
			auto Result = Array.PadToNumDefaulted(4);
			check(Result == 0);
			check(Array.Num() == 5);
			check(Array == (ArrayType{ ElementType{}, ElementType{}, ElementType{}, ElementType{}, ElementType{} }));
		}
		{
			auto Result = Array.PadToNum(4, PaddingValue1);
			check(Result == 0);
			check(Array.Num() == 5);
			check(Array == (ArrayType{ ElementType{}, ElementType{}, ElementType{}, ElementType{}, ElementType{} }));
		}

		// Check adding exact padding
		{
			auto Result = Array.PadToNumDefaulted(5);
			check(Result == 0);
			check(Array.Num() == 5);
			check(Array == (ArrayType{ ElementType{}, ElementType{}, ElementType{}, ElementType{}, ElementType{} }));
		}
		{
			auto Result = Array.PadToNum(5, PaddingValue1);
			check(Result == 0);
			check(Array.Num() == 5);
			check(Array == (ArrayType{ ElementType{}, ElementType{}, ElementType{}, ElementType{}, ElementType{} }));
		}

		// Check adding 1 more padding value to non-empty arrays
		{
			auto Result = Array.PadToNumDefaulted(6);
			check(Result == 1);
			check(Array.Num() == 6);
			check(Array == (ArrayType{ ElementType{}, ElementType{}, ElementType{}, ElementType{}, ElementType{}, ElementType{} }));
		}
		{
			auto Result = Array.PadToNum(7, PaddingValue1);
			check(Result == 1);
			check(Array.Num() == 7);
			check(Array == (ArrayType{ ElementType{}, ElementType{}, ElementType{}, ElementType{}, ElementType{}, ElementType{}, PaddingValue1 }));
		}

		// Check adding multiple padding to non-empty arrays
		{
			auto Result = Array.PadToNumDefaulted(10);
			check(Result == 3);
			check(Array.Num() == 10);
			check(Array == (ArrayType{ ElementType{}, ElementType{}, ElementType{}, ElementType{}, ElementType{}, ElementType{}, PaddingValue1, ElementType{}, ElementType{}, ElementType{} }));
		}
		{
			auto Result = Array.PadToNum(14, PaddingValue2);
			check(Result == 4);
			check(Array.Num() == 14);
			check(Array == (ArrayType{ ElementType{}, ElementType{}, ElementType{}, ElementType{}, ElementType{}, ElementType{}, PaddingValue1, ElementType{}, ElementType{}, ElementType{}, PaddingValue2, PaddingValue2, PaddingValue2, PaddingValue2 }));
		}
	}

	template <typename MapType, typename KeyType>
	void RunMapTests()
	{
		MapType Map;

		int32 MaxNum = 0;
		SIZE_T MaxAllocatedSize = 0;
		const SIZE_T InitialAllocatedSize = Map.GetAllocatedSize();

		ContainerTestStats.Reset();

		auto CheckMap = [&Map]()
		{
			CheckContainerNum(Map);
			CheckContainerEnds(Map);
			CheckContainerElements(Map);
			CheckContainerSelfEquality(Map);
		};

		// Test Add and Remove
		// Subtract one to account for temporaries that will be created during an Add
		for (int32 Count = 0; Count < MAX_TEST_OBJECTS - 1; Count += MAX_TEST_OBJECTS_STEP)
		{
			for (int32 N = 0; N != Count; ++N)
			{
				Map.Add(TestKeyGenerator<KeyType>::GetKey(N), FContainerTestValueType(TEXT("New Value")));
				CheckMap();
			}
			MaxNum = Map.Num();
			MaxAllocatedSize = Map.GetAllocatedSize();

			for (int32 N = 0; N != Count; ++N)
			{
				check(Map.Remove(TestKeyGenerator<KeyType>::GetKey(N)) == 1);
				CheckMap();
			}

			check(Map.IsEmpty());

			for (int32 N = 0; N != Count; ++N)
			{
				Map.Add(TestKeyGenerator<KeyType>::GetKey((Count - 1) - N), FContainerTestValueType(TEXT("New Value")));
				CheckMap();
			}

			for (int32 N = 0; N != Count; ++N)
			{
				check(Map.Remove(TestKeyGenerator<KeyType>::GetKey(N)) == 1);
				CheckMap();
			}

			check(Map.IsEmpty());
		}

		// Test Empty and Shrink
		{
			// Test releasing memory allocations
			Map.Empty();
			CheckMap();
			check(Map.GetAllocatedSize() == InitialAllocatedSize);

			// Test integrity after re-growing container to MaxNum elements again
			for (int32 N = 0; N < MaxNum; ++N)
			{
				Map.Add(TestKeyGenerator<KeyType>::GetKey(N), FContainerTestValueType(TEXT("New Value")));
			}
			CheckMap();
			check(Map.GetAllocatedSize() == MaxAllocatedSize);

			// Test data integrity while removing and shrinking continously
			{
				SIZE_T PrevAllocatedSize = Map.GetAllocatedSize();
				for (int32 N = MaxNum - 1; N >= MaxNum / 4; --N)
				{
					check(Map.Remove(TestKeyGenerator<KeyType>::GetKey(N)) == 1);
					Map.Shrink();
					CheckMap();
					check(Map.GetAllocatedSize() <= PrevAllocatedSize);
					PrevAllocatedSize = Map.GetAllocatedSize();
				}
			}

			// Test removing and releasing remaining elements
			Map.Empty();
			check(Map.IsEmpty());
			check(Map.GetAllocatedSize() == InitialAllocatedSize);
		}

		// Test key iterators
		{
			static_assert(std::is_same_v<decltype((Map.CreateKeyIterator     (DeclVal<KeyType&>())->Key)),         KeyType&>);
			static_assert(std::is_same_v<decltype((Map.CreateKeyIterator     (DeclVal<KeyType&>())->Value)),       FContainerTestValueType&>);
			static_assert(std::is_same_v<decltype((Map.CreateConstKeyIterator(DeclVal<KeyType&>())->Key)),   const KeyType&>);
			static_assert(std::is_same_v<decltype((Map.CreateConstKeyIterator(DeclVal<KeyType&>())->Value)), const FContainerTestValueType&>);

			const TCHAR* RegularValue  = TEXT("Regular");
			const TCHAR* ReplacedValue = TEXT("Replaced");

			for (int32 Count = 0; Count < MAX_TEST_OBJECTS - 1; Count += MAX_TEST_OBJECTS_STEP)
			{
				Map.Empty();
				CheckMap();

				for (int32 N = 0; N != Count; ++N)
				{
					Map.Add(TestKeyGenerator<KeyType>::GetKey(N), FContainerTestValueType(RegularValue));
					CheckMap();
				}

				// Iterate over all possible keys, and some before/after the range [0, Count) that won't exist
				for (int32 KeyValue = -2; KeyValue < Count + 2; ++KeyValue)
				{
					KeyType Key = TestKeyGenerator<KeyType>::GetKey(KeyValue);

					// Check that at most one key is found by the const key iterator
					const KeyType* FoundConstKey = nullptr;
					for (auto It = Map.CreateConstKeyIterator(Key); It; ++It)
					{
						check(!FoundConstKey);
						check(It->Key == Key);
						check(Map.Get(It.GetId()) == *It);
						FoundConstKey = &It->Key;
					}

					// Check that at most one key is found by the key iterator, and that we can mutate the value via one
					KeyType* FoundKey = nullptr;
					for (auto It = Map.CreateKeyIterator(Key); It; ++It)
					{
						check(!FoundKey);
						check(It->Key == Key);
						check(Map.Get(It.GetId()) == *It);
						FoundKey = &It->Key;
						It->Value.Str = ReplacedValue;
					}

					// Check that the key iterators found the right element, if any
					check(FoundKey == FoundConstKey);
					if (FoundConstKey)
					{
						check(KeyValue >= 0 && KeyValue < Count);
						check(Map[Key].Str == ReplacedValue);
					}
					else
					{
						check(KeyValue < 0 || KeyValue >= Count);
					}
				}
			}
		}
	}

	template <typename SetType, typename KeyType, bool bSorted = false>
	void RunSetTests()
	{
		SetType Set;

		int32 MaxNum = 0;
		SIZE_T MaxAllocatedSize = 0;
		const SIZE_T InitialAllocatedSize = Set.GetAllocatedSize();

		ContainerTestStats.Reset();

		auto CheckSet = [](SetType& Set)
		{
			CheckContainerNum(Set);
			CheckContainerEnds(Set);
			CheckContainerElements(Set);
			CheckContainerSelfEquality(Set);
			if constexpr (bSorted)
			{
				CheckContainerSorted(Set);
			}
		};

		// Test Add and Remove
		// Subtract one to account for temporaries that will be created during an Add
		for (int32 Count = 0; Count < MAX_TEST_OBJECTS - 1; Count += MAX_TEST_OBJECTS_STEP)
		{
			for (int32 N = 0; N != Count; ++N)
			{
				Set.Add(TestKeyGenerator<KeyType>::GetKey(N));
				CheckSet(Set);
			}
			MaxNum = Set.Num();
			MaxAllocatedSize = Set.GetAllocatedSize();

			for (int32 N = 0; N != Count; ++N)
			{
				auto* ElementPtr = Set.FindArbitraryElement();
				check(ElementPtr);
				check(Set.Remove(*ElementPtr) == 1);
				CheckSet(Set);
			}

			check(Set.IsEmpty());

			for (int32 N = 0; N != Count; ++N)
			{
				Set.Add(TestKeyGenerator<KeyType>::GetKey((Count - 1) - N));
				CheckSet(Set);
			}

			for (int32 N = 0; N != Count; ++N)
			{
				auto* ElementPtr = Set.FindArbitraryElement();
				check(ElementPtr);
				check(Set.Remove(*ElementPtr) == 1);
				CheckSet(Set);
			}

			check(Set.IsEmpty());
		}

		// Test Empty and Shrink
		{
			// Test releasing memory allocations
			Set.Empty();
			CheckSet(Set);
			check(Set.GetAllocatedSize() == InitialAllocatedSize);

			// Test integrity after re-growing container to MaxNum elements again
			for (int32 N = 0; N < MaxNum; ++N)
			{
				Set.Add(TestKeyGenerator<KeyType>::GetKey(N));
			}
			CheckSet(Set);
			check(Set.GetAllocatedSize() == MaxAllocatedSize);

			// Test data integrity while removing and shrinking continously
			{
				SIZE_T PrevAllocatedSize = Set.GetAllocatedSize();
				for (int32 N = MaxNum - 1; N >= MaxNum / 4; --N)
				{
					auto* ElementPtr = Set.FindArbitraryElement();
					check(ElementPtr);
					Set.Remove(*ElementPtr);
					Set.Shrink();
					CheckSet(Set);
					check(Set.GetAllocatedSize() <= PrevAllocatedSize);
					PrevAllocatedSize = Set.GetAllocatedSize();
				}
			}

			// Test removing and releasing remaining elements
			Set.Empty();
			check(Set.IsEmpty());
			check(Set.GetAllocatedSize() == InitialAllocatedSize);
		}

		// Test iterators
		{
			static_assert(std::is_same_v<decltype(*Set.CreateIterator     ()),         KeyType&>);
			static_assert(std::is_same_v<decltype(*Set.CreateConstIterator()),   const KeyType&>);

			for (int32 Count = 0; Count < MAX_TEST_OBJECTS - 1; Count += MAX_TEST_OBJECTS_STEP)
			{
				Set.Empty();
				CheckSet(Set);

				for (int32 N = 0; N != Count; ++N)
				{
					Set.Add(TestKeyGenerator<KeyType>::GetKey(N));
					CheckSet(Set);
				}

				// Check that the const iterator finds all the values
				SetType Clone = Set;
				for (auto It = Set.CreateConstIterator(); It; ++It)
				{
					check(Clone.Contains(*It));
					check(Clone.Remove(*It) == 1);
				}
				check(Clone.IsEmpty());

				// Check that the non-const iterator finds all the values
				Clone = Set;
				for (auto It = Set.CreateConstIterator(); It; ++It)
				{
					check(Clone.Contains(*It));
					check(Clone.Remove(*It) == 1);
				}
				check(Clone.IsEmpty());
			}
		}

		// Test bulk insertion
		{
			auto TestAppend = [&CheckSet](std::initializer_list<KeyType> InitSet, std::initializer_list<KeyType> Append, std::initializer_list<KeyType> ExpectedResult)
			{
				auto CheckSetEquals = [&CheckSet](SetType& Set, std::initializer_list<KeyType> ExpectedResult)
				{
					// Check set integrity
					CheckSet(Set);

					// Check
					if constexpr (bSorted)
					{
						TArray<KeyType> ExpectedResultArray = ExpectedResult;
						ExpectedResultArray.Sort();
						check(Set.Array() == ExpectedResultArray);
					}
					else
					{
						// Check every expected result value is there and no more
						for (const KeyType& Key : ExpectedResult)
						{
							check(Set.Remove(Key) == 1);
						}
						check(Set.IsEmpty());
					}
				};

				SetType DestSet;
				TArray<KeyType> SrcArray = Append;

				// Check append by copy
				{
					DestSet = InitSet;

					// Copy elements to set
					DestSet.Append(SrcArray);

					// Check that the source is unchanged
					check(SrcArray == TArray<KeyType>(Append));

					// Check that the set contains the expected values (this may mutate the set)
					CheckSetEquals(DestSet, ExpectedResult);
				}

				// Check append by move
				{
					DestSet = InitSet;

					// Copy elements to set
					DestSet.Append(SrcArray);

					// Check that the source is unchanged
					check(SrcArray == TArray<KeyType>(Append));

					// Check that the set contains the expected values (this may mutate the set)
					CheckSetEquals(DestSet, ExpectedResult);
				}
			};

			// Copy unique ordered elements to empty set
			const KeyType Key1 = TestKeyGenerator<KeyType>::GetKey(1);
			const KeyType Key2 = TestKeyGenerator<KeyType>::GetKey(2);
			const KeyType Key3 = TestKeyGenerator<KeyType>::GetKey(3);
			const KeyType Key4 = TestKeyGenerator<KeyType>::GetKey(4);
			const KeyType Key5 = TestKeyGenerator<KeyType>::GetKey(5);
			const KeyType Key7 = TestKeyGenerator<KeyType>::GetKey(7);
			TestAppend(
				{},
				{ Key1, Key2, Key3, Key4, Key5 },
				{ Key1, Key2, Key3, Key4, Key5 }
			);

			// Copy unique unordered elements to empty set
			TestAppend(
				{},
				{ Key3, Key5, Key1, Key2, Key4 },
				{ Key1, Key2, Key3, Key4, Key5 }
			);

			// Copy duplicate ordered elements to empty set
			TestAppend(
				{},
				{ Key1, Key1, Key2, Key3, Key3, Key3, Key4, Key5, Key5 },
				{ Key1, Key2, Key3, Key4, Key5 }
			);

			// Copy duplicate unordered elements to empty set
			TestAppend(
				{},
				{ Key3, Key5, Key1, Key1, Key4, Key2, Key5, Key4, Key3, Key1 },
				{ Key1, Key2, Key3, Key4, Key5 }
			);

			// Copy unique ordered elements to populated set
			TestAppend(
				{ Key1, Key5, Key7 },
				{ Key1, Key2, Key3, Key4, Key5 },
				{ Key1, Key2, Key3, Key4, Key5, Key7 }
			);

			// Copy unique unordered elements to populated set
			TestAppend(
				{ Key1, Key5, Key7 },
				{ Key3, Key5, Key1, Key2, Key4 },
				{ Key1, Key2, Key3, Key4, Key5, Key7 }
			);

			// Copy duplicate ordered elements to populated set
			TestAppend(
				{ Key1, Key5, Key7 },
				{ Key1, Key1, Key2, Key3, Key3, Key3, Key4, Key5, Key5 },
				{ Key1, Key2, Key3, Key4, Key5, Key7 }
			);

			// Copy duplicate unordered elements to populated set
			TestAppend(
				{ Key1, Key5, Key7 },
				{ Key3, Key5, Key1, Key1, Key4, Key2, Key5, Key4, Key3, Key1 },
				{ Key1, Key2, Key3, Key4, Key5, Key7 }
			);
		}
	}

	// Test container element address consistency when using SortFreeList
	// (see TSparseArray::SortFreeList for comments)
	template <typename MapType, typename KeyType>
	void RunMapConsistencyTests()
	{
		MapType Map;

		{
			// Add 3 elements, then remove 2 in the same order they were added
			const KeyType Key0 = TestKeyGenerator<KeyType>::GetKey(0);
			const KeyType Key1 = TestKeyGenerator<KeyType>::GetKey(1);
			const KeyType Key2 = TestKeyGenerator<KeyType>::GetKey(2);
			const FContainerTestValueType Value = FContainerTestValueType(TEXT("New Value"));
			Map.Add(Key0, Value);
			Map.Add(Key1, Value);
			Map.Add(Key2, Value);
			const FContainerTestValueType* ValuePtr0 = Map.Find(Key0);
			const FContainerTestValueType* ValuePtr1 = Map.Find(Key1);
			const FContainerTestValueType* ValuePtr2 = Map.Find(Key2);
			check(ValuePtr0 != nullptr);
			check(ValuePtr1 != nullptr);
			check(ValuePtr2 != nullptr);
			check(Map.Remove(Key1) == 1);
			check(Map.Remove(Key2) == 1);

			if constexpr (requires { Map.SortFreeList(); })
			{
				// Re-add the 2 elements in the same order. Without the call to SortFreeList()
				// the elements would end up in a different locations/order compared to the
				// original insertions. SortFreeList() should ensure that re-adding the elements
				// gives use the same container layout as long as we perform the same operations
				// in the same order as before.
				Map.SortFreeList();
				Map.Add(Key1, Value);
				Map.Add(Key2, Value);
				const FContainerTestValueType* NewValuePtr0 = Map.Find(Key0);
				const FContainerTestValueType* NewValuePtr1 = Map.Find(Key1);
				const FContainerTestValueType* NewValuePtr2 = Map.Find(Key2);

				check(ValuePtr0 == NewValuePtr0);
				check(ValuePtr1 == NewValuePtr1);
				check(ValuePtr2 == NewValuePtr2);
			}
		}
	}

	template <typename Container>
	void RunEmptyContainerSelfEqualityTest()
	{
		Container Cont;
		CheckContainerSelfEquality(Cont);
	}

	template <typename MapType, typename KeyType>
	void RunMapPerformanceTest(const FString& Description, int32 NumObjects, int32 NumOperations)
	{
		ContainerTestStats.Reset();

		MapType Map;
		FRandomStream RandomStream(RANDOM_SEED);

		// Prep keys, not part of performance test
		TArray<KeyType> KeyArray;
		KeyArray.Reserve(NumObjects);

		for (int32 i = 0; i < NumObjects; i++)
		{
			KeyArray.Add(TestKeyGenerator<KeyType>::GetKey(i));
		}

		for (int32 i = 0; i < NumObjects; i++)
		{
			int32 SwapIndex = RandomStream.RandRange(0, NumObjects - 1);
			if (i != SwapIndex)
			{
				KeyArray.Swap(i, SwapIndex);
			}
		}

		FScopeLogTime LogTimePtr(*FString::Printf(TEXT("%s objects=%d count=%d"), *Description, NumObjects, NumOperations), nullptr, FScopeLogTime::ScopeLog_Milliseconds);

		// Add elements in stably randomized order
		for (int32 i = 0; i < NumObjects; i++)
		{
			Map.Add(KeyArray[i], FString("New Value"));
		}

		// Now do searches
		for (int32 i = 0; i < NumOperations; i++)
		{
			KeyType& Key = KeyArray[RandomStream.RandRange(0, NumObjects - 1)];

			FString* FoundValue = Map.Find(Key);
			check(FoundValue);
		}
	}

	template <typename SetType, typename KeyType>
	void RunSetPerformanceTest(const FString& Description, int32 NumObjects, int32 NumOperations)
	{
		ContainerTestStats.Reset();

		SetType Set;
		FRandomStream RandomStream(RANDOM_SEED);

		// Prep keys, not part of performance test
		TArray<KeyType> KeyArray;
		KeyArray.Reserve(NumObjects);

		for (int32 i = 0; i < NumObjects; i++)
		{
			KeyArray.Add(TestKeyGenerator<KeyType>::GetKey(i));
		}

		for (int32 i = 0; i < NumObjects; i++)
		{
			int32 SwapIndex = RandomStream.RandRange(0, NumObjects - 1);
			if (i != SwapIndex)
			{
				KeyArray.Swap(i, SwapIndex);
			}
		}

		FScopeLogTime LogTimePtr(*FString::Printf(TEXT("%s objects=%d count=%d"), *Description, NumObjects, NumOperations), nullptr, FScopeLogTime::ScopeLog_Milliseconds);

		// Add elements in stably randomized order
		for (int32 i = 0; i < NumObjects; i++)
		{
			Set.Add(KeyArray[i]);
		}

		// Now do searches
		for (int32 i = 0; i < NumOperations; i++)
		{
			KeyType& Key = KeyArray[RandomStream.RandRange(0, NumObjects - 1)];

			bool FoundValue = Set.Contains(Key);
			check(FoundValue);
		}
	}

	template<typename TValueType>
	struct FCaseSensitiveLookupKeyFuncs : BaseKeyFuncs<TValueType, FString>
	{
		static FORCEINLINE const FString& GetSetKey(const TPair<FString, TValueType>& Element)
		{
			return Element.Key;
		}
		static FORCEINLINE bool Matches(const FString& A, const FString& B)
		{
			return A.Equals(B, ESearchCase::CaseSensitive);
		}
		static FORCEINLINE uint32 GetKeyHash(const FString& Key)
		{
			return FCrc::StrCrc32<TCHAR>(*Key);
		}
	};

	void RunGetRefTests()
	{
		{
			TArray<FString> Arr;
			FString* Str1 = &Arr.AddDefaulted_GetRef();
			check(Str1->IsEmpty());
			check(Str1 == &Arr.Last());

			FString* Str2 = &Arr.AddZeroed_GetRef();
			check(Str2->IsEmpty());
			check(Str2 == &Arr.Last());

			FString* Str3 = &Arr.Add_GetRef(TEXT("Abc"));
			check(*Str3 == TEXT("Abc"));
			check(Str3 == &Arr.Last());

			FString* Str4 = &Arr.Emplace_GetRef(TEXT("Def"));
			check(*Str4 == TEXT("Def"));
			check(Str4 == &Arr.Last());

			FString* Str5 = &Arr.EmplaceAt_GetRef(3, TEXT("Ghi"));
			check(*Str5 == TEXT("Ghi"));
			check(Str5 == &Arr[3]);

			FString* Str6 = &Arr.InsertDefaulted_GetRef(2);
			check(Str6->IsEmpty());
			check(Str6 == &Arr[2]);

			FString* Str7 = &Arr.InsertZeroed_GetRef(4);
			check(Str7->IsEmpty());
			check(Str7 == &Arr[4]);
		}
	}
}

template <typename T>
static void RunSetTestsWithType()
{
	RunSetTests<TSet<T>, T>();
	RunSetTests<TSet<T, DefaultKeyFuncs<T>, TInlineSetAllocator<32>>, T>();
	RunSetTests<TSet<T, DefaultKeyFuncs<T>, TFixedSetAllocator<64>>, T>();

	RunSetTests<TSortedSet<T>, T, true>();
	RunSetTests<TSortedSet<T, TInlineAllocator<32>>, T, true>();
	RunSetTests<TSortedSet<T, TFixedAllocator<64>>, T, true>();
}

template <typename SetType, typename KeyType>
static void RunSetEmplaceSingleTestsWithType()
{
	// Test Emplace(Object), Contains, IsValidId, Remove.
	for (int32 Count = 1; Count < MAX_TEST_OBJECTS; Count += MAX_TEST_OBJECTS_STEP)
	{
		SetType Set;

		for (int32 N = 0; N != Count; ++N)
		{
			check(Set.Num() == N);

			KeyType Key = TestKeyGenerator<KeyType>::GetKey(N);

			bool bAlreadyInSet = false;
			FSetElementId ElementId = Set.Emplace(Key, &bAlreadyInSet);
			check(!bAlreadyInSet);
			check(Set.IsValidId(ElementId));
			check(Set.Get(ElementId) == Key);
			check(Set.Contains(Key));

			FSetElementId RepeatedElementId = Set.Emplace(Key, &bAlreadyInSet);
			check(bAlreadyInSet);
			check(ElementId == RepeatedElementId);
			check(Set.IsValidId(ElementId));
			check(Set.Get(ElementId) == Key);
			check(Set.Contains(Key));
		}

		check(Set.Num() == Count);

		for (int32 N = 0; N != Count; ++N)
		{
			check(Set.Num() == Count - N);

			KeyType Key = TestKeyGenerator<KeyType>::GetKey(N);

			check(Set.Contains(Key));
			check(Set.Remove(Key) == 1);

			check(!Set.Contains(Key));
			check(Set.Remove(Key) == 0);
		}

		check(Set.IsEmpty());
		check(Set.Num() == 0);
	}

	// Test Emplace(InPlace, Value), Contains, IsValidId, Remove.
	for (int32 Count = 1; Count < MAX_TEST_OBJECTS; Count += MAX_TEST_OBJECTS_STEP)
	{
		SetType Set;

		for (int32 N = 0; N != Count; ++N)
		{
			check(Set.Num() == N);

			KeyType Key = TestKeyGenerator<KeyType>::GetKey(N);

			TPair<FSetElementId, bool> EmplaceResult = Set.Emplace(InPlace, Key);
			const auto& [ElementId, bAlreadyInSet] = EmplaceResult;
			check(!bAlreadyInSet);
			check(Set.IsValidId(ElementId));
			check(Set.Get(ElementId) == Key);
			check(Set.Contains(Key));

			TPair<FSetElementId, bool> RepeatedEmplaceResult = Set.Emplace(InPlace, Key);
			const auto& [RepeatedElementId, bRepeatedAlreadyInSet] = RepeatedEmplaceResult;
			check(bRepeatedAlreadyInSet);
			check(ElementId == RepeatedElementId);
			check(Set.IsValidId(ElementId));
			check(Set.Get(ElementId) == Key);
			check(Set.Contains(Key));
		}

		check(Set.Num() == Count);

		for (int32 N = 0; N != Count; ++N)
		{
			check(Set.Num() == Count - N);

			KeyType Key = TestKeyGenerator<KeyType>::GetKey(N);

			check(Set.Contains(Key));
			check(Set.Remove(Key) == 1);

			check(!Set.Contains(Key));
			check(Set.Remove(Key) == 0);
		}

		check(Set.IsEmpty());
		check(Set.Num() == 0);
	}
}

template <typename SetType, typename Key0Type, typename Key1Type>
static void RunSetEmplaceDoubleTestsWithType()
{
	// Test Emplace(InPlace, A, B), Contains, IsValidId, Remove.
	for (int32 Count = 1; Count < MAX_TEST_OBJECTS; Count += MAX_TEST_OBJECTS_STEP)
	{
		SetType Set;

		for (int32 N = 0; N != Count; ++N)
		{
			check(Set.Num() == N);

			Key0Type Key0 = TestKeyGenerator<Key0Type>::GetKey(N);
			Key1Type Key1 = TestKeyGenerator<Key1Type>::GetKey(N + 100);
			TPair<Key0Type, Key1Type> Key{Key0, Key1};

			TPair<FSetElementId, bool> EmplaceResult = Set.Emplace(InPlace, Key0, Key1);
			const auto& [ElementId, bAlreadyInSet] = EmplaceResult;
			check(!bAlreadyInSet);
			check(Set.IsValidId(ElementId));
			check(Set.Get(ElementId) == Key);
			check(Set.Contains(Key));

			TPair<FSetElementId, bool> RepeatedEmplaceResult = Set.Emplace(InPlace, Key0, Key1);
			const auto& [RepeatedElementId, bRepeatedAlreadyInSet] = RepeatedEmplaceResult;
			check(bRepeatedAlreadyInSet);
			check(ElementId == RepeatedElementId);
			check(Set.IsValidId(ElementId));
			check(Set.Get(ElementId) == Key);
			check(Set.Contains(Key));
		}

		check(Set.Num() == Count);

		for (int32 N = 0; N != Count; ++N)
		{
			check(Set.Num() == Count - N);

			Key0Type Key0 = TestKeyGenerator<Key0Type>::GetKey(N);
			Key1Type Key1 = TestKeyGenerator<Key1Type>::GetKey(N + 100);
			TPair<Key0Type, Key1Type> Key{Key0, Key1};

			check(Set.Contains(Key));
			check(Set.Remove(Key) == 1);

			check(!Set.Contains(Key));
			check(Set.Remove(Key) == 0);
		}

		check(Set.IsEmpty());
		check(Set.Num() == 0);
	}
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FContainersSmokeTest, "System.Core.Containers.Smoke", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool FContainersSmokeTest::RunTest( const FString& Parameters )
{
	RunMapTests<TMap<int32, FContainerTestValueType>, int32>();
	RunMapTests<TMap<int32, FContainerTestValueType, TInlineSetAllocator<32>>, int32>();
	RunMapTests<TMap<int32, FContainerTestValueType, TFixedSetAllocator<64>>, int32>();

	RunMapTests<TSortedMap<int32, FContainerTestValueType>, int32>();
	RunMapTests<TSortedMap<int32, FContainerTestValueType, TInlineAllocator<32>>, int32>();
	RunMapTests<TSortedMap<int32, FContainerTestValueType, TFixedAllocator<64>>, int32>();

	RunSetTestsWithType<int32>();

	RunSetEmplaceSingleTestsWithType<TSet<int32>, int32>();
	RunSetEmplaceSingleTestsWithType<TSet<FString>, FString>();

	RunSetEmplaceDoubleTestsWithType<TSet<TPair<int32, FString>>, int32, FString>();
	RunSetEmplaceDoubleTestsWithType<TSet<TPair<FName, FName>>, FName, FName>();

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FContainersFullTest, "System.Core.Containers.Full", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
bool FContainersFullTest::RunTest(const FString& Parameters)
{
	RunPadToNumTests<TArray<int32>>(5, 7);
	RunPadToNumTests<TArray<FString>>("Hello", "World");

	RunMapTests<TMap<int32, FContainerTestValueType>, int32>();
	RunMapTests<TMap<FName, FContainerTestValueType>, FName>();
	RunMapTests<TMap<FString, FContainerTestValueType>, FString>();
	RunMapTests<TMap<int32, FContainerTestValueType, TInlineSetAllocator<32>>, int32>();
	RunMapTests<TMap<int32, FContainerTestValueType, TInlineSetAllocator<64>>, int32>();
	RunMapTests<TMap<int32, FContainerTestValueType, TFixedSetAllocator<64>>, int32>();
	RunMapTests<TMap<FString, FContainerTestValueType, FDefaultSetAllocator, FCaseSensitiveLookupKeyFuncs<FContainerTestValueType>>, FString>();

	RunMapConsistencyTests<TMap<int32, FContainerTestValueType>, int32>();
	RunMapConsistencyTests<TMap<int32, FContainerTestValueType, TInlineSetAllocator<32>>, int32>();
	RunMapConsistencyTests<TMap<int32, FContainerTestValueType, TFixedSetAllocator<64>>, int32>();

	RunMapTests<TSortedMap<int32, FContainerTestValueType>, int32>();
	RunMapTests<TSortedMap<FName, FContainerTestValueType, FDefaultAllocator, FNameLexicalLess>, FName>();
	RunMapTests<TSortedMap<FString, FContainerTestValueType>, FString>();
	RunMapTests<TSortedMap<FString, FContainerTestValueType, TInlineAllocator<64>>, FString>();

	RunEmptyContainerSelfEqualityTest<TArray<int32>>();

	RunSetTestsWithType<int32>();
	RunSetTestsWithType<FString>();
	RunSetTestsWithType<TPair<int32, int16>>();
	RunSetTestsWithType<TPair<FString, FString>>();

	RunSetEmplaceSingleTestsWithType<TSet<int32>, int32>();
	RunSetEmplaceSingleTestsWithType<TSet<FString>, FString>();
	RunSetEmplaceSingleTestsWithType<TSet<FName>, FName>();

	RunSetEmplaceDoubleTestsWithType<TSet<TPair<FString, int32>>, FString, int32>();
	RunSetEmplaceDoubleTestsWithType<TSet<TPair<int32, FString>>, int32, FString>();
	RunSetEmplaceDoubleTestsWithType<TSet<TPair<FName, FName>>, FName, FName>();

	// Verify use of FName index sorter with SortedMap

	TSortedMap<FName, int32, FDefaultAllocator, FNameFastLess> NameMap;
	NameMap.Add(NAME_NameProperty);
	NameMap.Add(NAME_FloatProperty);
	NameMap.Add(NAME_None);
	NameMap.Add(NAME_IntProperty);

	auto It = NameMap.CreateConstIterator();

	check(It->Key == NAME_None); ++It;
	check(It->Key == NAME_IntProperty); ++It;
	check(It->Key == NAME_FloatProperty); ++It;
	check(It->Key == NAME_NameProperty); ++It;
	check(!It);

	RunGetRefTests();

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FContainerPerformanceTest, "System.Core.Containers.Performance", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
bool FContainerPerformanceTest::RunTest(const FString& Parameters)
{
	RunMapPerformanceTest<TMap<int32, FString>, int32>(TEXT("TMap int32"), 1, 1000000);
	RunMapPerformanceTest<TMap<int32, FString>, int32>(TEXT("TMap int32"), 10, 1000000);
	RunMapPerformanceTest<TMap<int32, FString>, int32>(TEXT("TMap int32"), 100, 1000000);
	RunMapPerformanceTest<TMap<int32, FString>, int32>(TEXT("TMap int32"), 1000, 1000000);
	RunMapPerformanceTest<TMap<int32, FString>, int32>(TEXT("TMap int32"), 10000, 1000000);

	RunMapPerformanceTest<TMap<FName, FString>, FName>(TEXT("TMap FName"), 1, 1000000);
	RunMapPerformanceTest<TMap<FName, FString>, FName>(TEXT("TMap FName"), 10, 1000000);
	RunMapPerformanceTest<TMap<FName, FString>, FName>(TEXT("TMap FName"), 100, 1000000);
	RunMapPerformanceTest<TMap<FName, FString>, FName>(TEXT("TMap FName"), 1000, 1000000);
	RunMapPerformanceTest<TMap<FName, FString>, FName>(TEXT("TMap FName"), 10000, 1000000);

	RunMapPerformanceTest<TMap<FString, FString>, FString>(TEXT("TMap FString"), 1, 1000000);
	RunMapPerformanceTest<TMap<FString, FString>, FString>(TEXT("TMap FString"), 10, 1000000);
	RunMapPerformanceTest<TMap<FString, FString>, FString>(TEXT("TMap FString"), 100, 1000000);
	RunMapPerformanceTest<TMap<FString, FString>, FString>(TEXT("TMap FString"), 1000, 1000000);
	RunMapPerformanceTest<TMap<FString, FString>, FString>(TEXT("TMap FString"), 10000, 1000000);

	RunMapPerformanceTest<TSortedMap<int32, FString>, int32>(TEXT("TSortedMap int32"), 1, 1000000);
	RunMapPerformanceTest<TSortedMap<int32, FString>, int32>(TEXT("TSortedMap int32"), 10, 1000000);
	RunMapPerformanceTest<TSortedMap<int32, FString>, int32>(TEXT("TSortedMap int32"), 100, 1000000);
	RunMapPerformanceTest<TSortedMap<int32, FString>, int32>(TEXT("TSortedMap int32"), 1000, 1000000);
	RunMapPerformanceTest<TSortedMap<int32, FString>, int32>(TEXT("TSortedMap int32"), 10000, 1000000);

	RunMapPerformanceTest<TSortedMap<FName, FString, FDefaultAllocator, FNameLexicalLess>, FName>(TEXT("TSortedMap FName"), 1, 1000000);
	RunMapPerformanceTest<TSortedMap<FName, FString, FDefaultAllocator, FNameLexicalLess>, FName>(TEXT("TSortedMap FName"), 10, 1000000);
	RunMapPerformanceTest<TSortedMap<FName, FString, FDefaultAllocator, FNameLexicalLess>, FName>(TEXT("TSortedMap FName"), 100, 1000000);
	RunMapPerformanceTest<TSortedMap<FName, FString, FDefaultAllocator, FNameLexicalLess>, FName>(TEXT("TSortedMap FName"), 1000, 1000000);
	RunMapPerformanceTest<TSortedMap<FName, FString, FDefaultAllocator, FNameLexicalLess>, FName>(TEXT("TSortedMap FName"), 10000, 1000000);

	RunMapPerformanceTest<TSortedMap<FString, FString>, FString>(TEXT("TSortedMap FString"), 1, 1000000);
	RunMapPerformanceTest<TSortedMap<FString, FString>, FString>(TEXT("TSortedMap FString"), 10, 1000000);
	RunMapPerformanceTest<TSortedMap<FString, FString>, FString>(TEXT("TSortedMap FString"), 100, 1000000);
	RunMapPerformanceTest<TSortedMap<FString, FString>, FString>(TEXT("TSortedMap FString"), 1000, 1000000);
	RunMapPerformanceTest<TSortedMap<FString, FString>, FString>(TEXT("TSortedMap FString"), 10000, 1000000);

	RunSetPerformanceTest<TSet<FName>, FName>(TEXT("TSet FName"), 1, 1000000);
	RunSetPerformanceTest<TSet<FName>, FName>(TEXT("TSet FName"), 10, 1000000);
	RunSetPerformanceTest<TSet<FName>, FName>(TEXT("TSet FName"), 100, 1000000);

	RunSetPerformanceTest<TArray<FName>, FName>(TEXT("TArray FName"), 1, 1000000);
	RunSetPerformanceTest<TArray<FName>, FName>(TEXT("TArray FName"), 10, 1000000);
	RunSetPerformanceTest<TArray<FName>, FName>(TEXT("TArray FName"), 100, 1000000);

	return true;
}

namespace ContainersTSetTest
{
	struct FRecorder
	{
		FRecorder(uint32 InKey = 0, uint32 InPayload=0)
			: Id(NextId++)
			, Key(InKey)
			, Payload(InPayload)
		{
		}
		FRecorder(const FRecorder& Other)
			: Id(NextId++)
			, Key(Other.Key)
			, Payload(Other.Payload)
			, NumCopies(Other.NumCopies+1)
			, NumMoves(Other.NumMoves)
		{
		}
		FRecorder(FRecorder&& Other)
			: Id(NextId++)
			, Key(Other.Key)
			, Payload(Other.Payload)
			, NumCopies(Other.NumCopies)
			, NumMoves(Other.NumMoves+1)
		{
		}

		bool operator==(const FRecorder& Other) const
		{
			return Key == Other.Key;
		}

		uint32 Id;
		uint32 Key;
		uint32 Payload;
		uint32 NumCopies = 0;
		uint32 NumMoves = 0;

		static uint32 NextId;
	};
	uint32 FRecorder::NextId = 0;

	uint32 GetTypeHash(const FRecorder& Recorder)
	{
		return Recorder.Key;
	}

}
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FContainersTSetTest, "System.Core.Containers.TSet", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
bool FContainersTSetTest::RunTest(const FString& Parameters)
{
	using namespace ContainersTSetTest;

	enum class EArgType
	{
		Copy,
		Move
	};
	enum class EHashType
	{
		Internal,
		PassedIn
	};

	for (EHashType HashType : { EHashType::Internal, EHashType::PassedIn})
	{
		for (EArgType ArgType : {EArgType::Copy, EArgType::Move})
		{
			FString FuncName;
			auto FullText = [HashType,ArgType,&FuncName](const TCHAR* Message)
			{
				const TCHAR* HashText = HashType == EHashType::Internal ? TEXT("") : TEXT("ByHash");
				const TCHAR* ArgText = ArgType == EArgType::Copy ? TEXT("const&") : TEXT("&&");
				return FString::Printf(TEXT("TSet::%s%s(%s) %s"), *FuncName, HashText, ArgText, Message);
			};

			// Test TSet::Add(const&), Add(&&), AddByHash(const&), AddByHash(&&)
			FuncName = TEXT("Add");
			{
				TSet<FRecorder> Set;
				FRecorder First(37, 43);
				bool bAlreadyInSet = true;

				if (HashType == EHashType::Internal)
					if (ArgType == EArgType::Copy)
						Set.Add(First, &bAlreadyInSet);
					else
						Set.Add(MoveTemp(First), &bAlreadyInSet);
				else
					if (ArgType == EArgType::Copy)
						Set.AddByHash(GetTypeHash(First), First, &bAlreadyInSet);
					else
						Set.AddByHash(GetTypeHash(First), MoveTemp(First), &bAlreadyInSet);
				TestFalse(FullText(TEXT("returns bAlreadyInSet==false for first add")), bAlreadyInSet);

				FRecorder* Found = Set.Find(First);
				if (ArgType == EArgType::Copy)
					TestTrue(FullText(TEXT("constructs a copy")), Found && Found->Id > First.Id && Found->NumCopies > 0 && Found->Payload == First.Payload);
				else
					TestTrue(FullText(TEXT("constructs a move")), Found && Found->Id > First.Id && Found->NumCopies == 0 && Found->NumMoves >= 1 && Found->Payload == First.Payload);

				uint32 FoundId = Found ? Found->Id : 0;
				Found = Set.Find(First);
				TestTrue(TEXT("Finding an element returns a reference, no copies"), Found && Found->Id == FoundId);

				FRecorder Second(37, 56);
				if (HashType == EHashType::Internal)
					if (ArgType == EArgType::Copy)
						Set.Add(Second, &bAlreadyInSet);
					else
						Set.Add(MoveTemp(Second), &bAlreadyInSet);
				else
					if (ArgType == EArgType::Copy)
						Set.AddByHash(GetTypeHash(Second), Second, &bAlreadyInSet);
					else
						Set.AddByHash(GetTypeHash(Second), MoveTemp(Second), &bAlreadyInSet);
				TestTrue(FullText(TEXT("returns bAlreadyInSet==true for second add")), bAlreadyInSet);
				Found = Set.Find(First);
				TestTrue(FullText(TEXT("with a duplicate key constructs a copy of the new key")), Found && Found->Id > Second.Id && Found->Payload == Second.Payload);
			}

			// Test TSet::FindOrAdd(const&), FindOrAdd(&&), FindOrAddByHash(const&), FindOrAddByHash(&&)
			FuncName = TEXT("FindOrAdd");
			{
				TSet<FRecorder> Set;
				FRecorder First(37, 43);
				bool bAlreadyInSet = true;

				FRecorder* FindOrAddResult;
				if (HashType == EHashType::Internal)
					if (ArgType == EArgType::Copy)
						FindOrAddResult = &Set.FindOrAdd(First, &bAlreadyInSet);
					else
						FindOrAddResult = &Set.FindOrAdd(MoveTemp(First), &bAlreadyInSet);
				else
					if (ArgType == EArgType::Copy)
						FindOrAddResult = &Set.FindOrAddByHash(GetTypeHash(First), First, &bAlreadyInSet);
					else
						FindOrAddResult = &Set.FindOrAddByHash(GetTypeHash(First), MoveTemp(First), &bAlreadyInSet);
				TestFalse(FullText(TEXT("returns bAlreadyInSet==false for first add")), bAlreadyInSet);
				if (ArgType == EArgType::Copy)
					TestTrue(FullText(TEXT("on the first constructs a copy")), FindOrAddResult->Id > First.Id && FindOrAddResult->NumCopies > 0 && FindOrAddResult->Payload == First.Payload);
				else
					TestTrue(FullText(TEXT("on the first constructs a move")), FindOrAddResult->Id > First.Id && FindOrAddResult->NumCopies == 0 && FindOrAddResult->NumMoves >= 1 && FindOrAddResult->Payload == First.Payload);
				uint32 FoundId = FindOrAddResult->Id;

				FRecorder* Found = Set.Find(First);
				TestTrue(FullText(TEXT("returns same value as future find")), Found&& Found->Id == FindOrAddResult->Id);
				Found = Set.Find(First);
				TestTrue(TEXT("Finding an element returns a reference, no copies"), Found && Found->Id == FoundId);

				FRecorder Second(37, 56);
				if (HashType == EHashType::Internal)
					if (ArgType == EArgType::Copy)
						FindOrAddResult = &Set.FindOrAdd(Second, &bAlreadyInSet);
					else
						FindOrAddResult = &Set.FindOrAdd(MoveTemp(Second), &bAlreadyInSet);
				else
					if (ArgType == EArgType::Copy)
						FindOrAddResult = &Set.FindOrAddByHash(GetTypeHash(Second), Second, &bAlreadyInSet);
					else
						FindOrAddResult = &Set.FindOrAddByHash(GetTypeHash(Second), MoveTemp(Second), &bAlreadyInSet);
				TestTrue(FullText(TEXT("returns bAlreadyInSet==true for second add")), bAlreadyInSet);
				TestTrue(FullText(TEXT("with a duplicate key keeps the original key, returned from FindOrAdd")), FindOrAddResult->Id == FoundId && FindOrAddResult->Payload == First.Payload);
				Found = Set.Find(First);
				TestTrue(FullText(TEXT("with a duplicate key keeps the original key, returned from future Finds")), Found->Id == FoundId && Found->Payload == First.Payload);
			}
		}
	}

	// FindArbitraryElement
	{
		{
			TSet<FString> Set;
			FString* Found = Set.FindArbitraryElement();
			TestNull(TEXT("FindArbitraryElement returns null on an empty set"), Found);
		}

		{
			constexpr int32 ElementsToAdd = 100;
			constexpr int32 IndexToKeep = 67;

			TSet<FString> Set;
			for (int32 I = 0; I != ElementsToAdd; ++I)
			{
				Set.Add(LexToString(I));
			}
			for (int32 I = 0; I != ElementsToAdd; ++I)
			{
				if (I != IndexToKeep)
				{
					check(Set.Remove(LexToString(I)) == 1);
				}
			}

			FString* Found = Set.FindArbitraryElement();
			if (Found)
			{
				TestNotNull(TEXT("FindArbitraryElement finds a value on a highly sparse set"), Found);
				TestEqual(TEXT("FindArbitraryElement finds the correct value on a highly sparse set"), *Found, LexToString(IndexToKeep));
			}

			Found = nullptr;
			check(Set.Remove(LexToString(IndexToKeep)) == 1);

			FString* Found2 = Set.FindArbitraryElement();
			TestNull(TEXT("FindArbitraryElement returns null on an emptied set"), Found2);
		}
	}

	return !HasAnyErrors();
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FContainersTMapTest, "System.Core.Containers.TMap", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
bool FContainersTMapTest::RunTest(const FString& Parameters)
{
	// FindArbitraryElement
	{
		{
			TMap<int32, FString> Map;
			TPair<int32, FString>* Found = Map.FindArbitraryElement();
			TestNull(TEXT("FindArbitraryElement returns null on an empty map"), Found);
		}

		{
			constexpr int32 ElementsToAdd = 100;
			constexpr int32 IndexToKeep = 23;

			TMap<int32, FString> Map;
			for (int32 I = 0; I != ElementsToAdd; ++I)
			{
				Map.Add(I, LexToString(I));
			}
			for (int32 I = 0; I != ElementsToAdd; ++I)
			{
				if (I != IndexToKeep)
				{
					check(Map.Remove(I) == 1);
				}
			}

			TPair<int32, FString>* Found = Map.FindArbitraryElement();

			TestNotNull(TEXT("FindArbitraryElement finds a value on a highly sparse map"), Found);
			if (Found)
			{
				TestEqual(TEXT("FindArbitraryElement finds the correct key on a highly sparse map"), Found->Key, IndexToKeep);
				TestEqual(TEXT("FindArbitraryElement finds the correct value on a highly sparse map"), Found->Value, LexToString(IndexToKeep));
			}

			Found = nullptr;
			check(Map.Remove(IndexToKeep) == 1);

			TPair<int32, FString>* Found2 = Map.FindArbitraryElement();
			TestNull(TEXT("FindArbitraryElement returns null on an emptied map"), Found2);
		}
	}

	return !HasAnyErrors();
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FContainersTSortedMapTest, "System.Core.Containers.TSortedMap", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
bool FContainersTSortedMapTest::RunTest(const FString& Parameters)
{
	// FindArbitraryElement
	{
		{
			TSortedMap<int32, FString> Map;
			TPair<int32, FString>* Found = Map.FindArbitraryElement();
			TestNull(TEXT("FindArbitraryElement returns null on an empty map"), Found);
		}

		{
			constexpr int32 ElementsToAdd = 100;
			constexpr int32 IndexToKeep = 23;

			TSortedMap<int32, FString> Map;
			for (int32 I = 0; I != ElementsToAdd; ++I)
			{
				Map.Add(I, LexToString(I));
			}
			for (int32 I = 0; I != ElementsToAdd; ++I)
			{
				if (I != IndexToKeep)
				{
					check(Map.Remove(I) == 1);
				}
			}

			TPair<int32, FString>* Found = Map.FindArbitraryElement();

			TestNotNull(TEXT("FindArbitraryElement finds a value on a highly sparse map"), Found);
			if (Found)
			{
				TestEqual(TEXT("FindArbitraryElement finds the correct key on a highly sparse map"), Found->Key, IndexToKeep);
				TestEqual(TEXT("FindArbitraryElement finds the correct value on a highly sparse map"), Found->Value, LexToString(IndexToKeep));
			}

			Found = nullptr;
			check(Map.Remove(IndexToKeep) == 1);

			TPair<int32, FString>* Found2 = Map.FindArbitraryElement();
			TestNull(TEXT("FindArbitraryElement returns null on an emptied map"), Found2);
		}
	}

	return !HasAnyErrors();
}

namespace UE::Private
{
	struct FHashTableHelper
	{
		// Given a hash table, a key and a list of values - test that the linked list of values for this key matches the passed in values
		// Also tests that the list is correctly terminated to ~0u
		// e.g.: 
		//		FHashTable Table;
		//		Table.Add(1, 1);
		//		Table.Add(1, 2);
		//		UE::Private::FHashTableHelper::TestHashList(TEXT("Description"), TestPtr, Table, 1, 1, 2);
		template<typename... Values>
		static void TestHashList(const TCHAR* TestString, FAutomationTestBase* Test, const FHashTable& InTable, uint32 First, uint32 Second, Values&&... Rest)
		{
			Test->TestEqual(TestString, InTable.First(First), Second);

			if constexpr(sizeof...(Rest) > 0)
			{
				TestHashListInner(TestString, Test, InTable, Second, Rest...);
			}
			else
			{
				// Test end of list is correctly terminated
				Test->TestEqual(TestString, InTable.Next(Second), ~0u);
			}
		}

		// Secondary test as the first value requires a call to FHashTable::First and all subsequents require FHashTable::Next
		template<typename... Values>
		static void TestHashListInner(const TCHAR* TestString, FAutomationTestBase* Test, const FHashTable& InTable, uint32 First, uint32 Second, Values&&... Rest)
		{
			Test->TestEqual(TestString, InTable.Next(First), Second);

			if constexpr(sizeof...(Rest) > 0)
			{
				TestHashListInner(TestString, Test, InTable, Second, Rest...);
			}
			else
			{
				// Test end of list is correctly terminated
				Test->TestEqual(TestString, InTable.Next(Second), ~0u);
			}
		}
	};
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FContainersFHashTableTest, "System.Core.Containers.FHashTable", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool FContainersFHashTableTest::RunTest(const FString& Parameters)
{
	FHashTable HashTable(1024, 16);

	// Resizing < size
	{
		// Key list for key == 1 -> [3, 10, 13, 2, 1]
		HashTable.Add(1, 1);
		HashTable.Add(1, 2);
		HashTable.Add(1, 13);
		HashTable.Add(1, 10);
		HashTable.Add(1, 3);

		UE::Private::FHashTableHelper::TestHashList(TEXT("Middle gap initial list is correct"), this, HashTable, 1, 3, 10, 13, 2, 1);
		
		// Key list for key == 1 -> [3, 2, 1]
		HashTable.Resize(8);

		UE::Private::FHashTableHelper::TestHashList(TEXT("Middle gap resized list is correct"), this, HashTable, 1, 3, 2, 1);

		HashTable.Clear(1024, 16);

		// Key list for key == 1 -> [1, 13, 12, 11, 10]
		HashTable.Add(1, 10);
		HashTable.Add(1, 11);
		HashTable.Add(1, 12);
		HashTable.Add(1, 13);
		HashTable.Add(1, 1);

		UE::Private::FHashTableHelper::TestHashList(TEXT("Open end list is correct"), this, HashTable, 1, 1, 13, 12, 11, 10);

		// Key list for key == 1 -> [1]
		HashTable.Resize(8);

		UE::Private::FHashTableHelper::TestHashList(TEXT("Open end list is correct after resize"), this, HashTable, 1, 1);

		HashTable.Clear(1024, 16);

		// Key list for key == 1 -> [13, 12, 11, 10, 1]
		HashTable.Add(1, 1);
		HashTable.Add(1, 10);
		HashTable.Add(1, 11);
		HashTable.Add(1, 12);
		HashTable.Add(1, 13);

		UE::Private::FHashTableHelper::TestHashList(TEXT("Open begin list is correct"), this, HashTable, 1, 13, 12, 11, 10, 1);

		// Key list for key == 1 -> [1]
		HashTable.Resize(8);

		UE::Private::FHashTableHelper::TestHashList(TEXT("Open begin list is correct after resize"), this, HashTable, 1, 1);
	}

	return !HasAnyErrors();
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FContainersTArrayTest, "System.Core.Containers.TArray", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool FContainersTArrayTest::RunTest(const FString& Parameters)
{
	// Move semantics
	{
		// Move array to another array of the same type
		{
			TArray<int32> From = { 1, 2, 3, 4, 5 };
			TArray<int32> To = MoveTemp(From);
			TestTrue(TEXT("Move constructing an array relocates the elements"), From.IsEmpty() && To == TArray<int32>{ 1, 2, 3, 4, 5 });
		}

		// Move array to another array of bitwise-compatible type
		{
			// We can transfer memory between arrays of signed and unsigned
			TArray<int32> From = { 1, 2, 3, 4, 5 };
			TArray<uint32> To(MoveTemp(From));
			TestTrue(TEXT("Move constructing an array relocates the elements with bitwise compatible elements"), From.IsEmpty() && To == TArray<uint32>{ 1, 2, 3, 4, 5 });
		}

		// Move array to another array of bitwise-incompatible type
		{
			// We can't transfer memory, but we can copy
			TArray<int32> From = { 1, 2, 3, 4, 5 };
			TArray<int64> To(MoveTemp(From));
			TestTrue(TEXT("Move constructing an array does not relocate the elements with bitwise incompatible elements"), From == TArray<int32>{ 1, 2, 3, 4, 5 } && To == TArray<int64>{ 1, 2, 3, 4, 5 });
		}

		// Move array of unqualified type to an array of qualified type
		{
			// We can transfer memory from an array of non-const to an array of const
			TArray<int32> From = { 1, 2, 3, 4, 5 };
			TArray<const int32> To(MoveTemp(From));
			TestTrue(TEXT("Move constructing an array when adding const relocates the elements"), From.IsEmpty() && To == TArray<const int32>{ 1, 2, 3, 4, 5 });
		}

		// Move array of qualified type to an array of unqualified type
		{
			// We can't transfer memory from an array of const to an array of non-const, because that would be const-incorrect
			TArray<const int32> From = { 1, 2, 3, 4, 5 };
			TArray<int32> To(MoveTemp(From));
			TestTrue(TEXT("Move constructing an array when removing conts copies the elements"), From == TArray<const int32>{ 1, 2, 3, 4, 5 } && To == TArray<int32>{ 1, 2, 3, 4, 5 });
		}
	}

	// Shrinking
	auto RunArrayShrinkTests = [this](auto InContainer, EAllowShrinking AllowsShrinking)
	{
		using Container = decltype(InContainer);
		{
			Container Array;
			Array.SetNum(1000);
			Array.SetNum(1);
			TestEqual(TEXT("SetNum handles shrinking correctly"), (AllowsShrinking == EAllowShrinking::Yes), (Array.Max() < 1000));
		}
		{
			Container Array;
			Array.SetNum(1000);
			Array.SetNumZeroed(1);
			TestEqual(TEXT("SetNumZeroed handles shrinking correctly"), (AllowsShrinking == EAllowShrinking::Yes), (Array.Max() < 1000));
		}
		{
			Container Array;
			Array.SetNum(1000);
			Array.SetNumUninitialized(1);
			TestEqual(TEXT("SetNumUninitialized handles shrinking correctly"), (AllowsShrinking == EAllowShrinking::Yes), (Array.Max() < 1000));
		}
		{
			Container Array;
			Array.SetNum(1000);
			for (int Count=0; Count<1000; ++Count)
			{
				Array.Pop();
			}
			TestEqual(TEXT("Pop handles shrinking correctly"), (AllowsShrinking == EAllowShrinking::Yes), (Array.Max() < 1000));
		}
		{
			Container Array;
			Array.SetNum(1000);
			for (int Index=999; Index>=0; --Index)
			{
				Array.RemoveAt(Index);
			}
			TestEqual(TEXT("RemoveAt(Index) handles shrinking correctly"), (AllowsShrinking == EAllowShrinking::Yes), (Array.Max() < 1000));
		}
		{
			Container Array;
			Array.SetNum(1000);
			for (int Index=999; Index>=0; --Index)
			{
				Array.RemoveAt(Index, 1);
			}
			TestEqual(TEXT("RemoveAt(Index, Count) handles shrinking correctly"), (AllowsShrinking == EAllowShrinking::Yes), (Array.Max() < 1000));
		}
		{
			Container Array;
			Array.SetNum(1000);
			for (int Count=0; Count<1000; ++Count)
			{
				Array.RemoveAtSwap(0);
			}
			TestEqual(TEXT("RemoveAtSwap(Index) handles shrinking correctly"), (AllowsShrinking == EAllowShrinking::Yes), (Array.Max() < 1000));
		}
		{
			Container Array;
			Array.SetNum(1000);
			for (int Count=0; Count<1000; ++Count)
			{
				Array.RemoveAtSwap(0, 1);
			}
			TestEqual(TEXT("RemoveAtSwap(Index, Count) handles shrinking correctly"), (AllowsShrinking == EAllowShrinking::Yes), (Array.Max() < 1000));
		}
		{
			Container Array;
			Array.SetNum(1000);
			for (int Count=0; Count<1000; ++Count)
			{
				Array.RemoveAtSwap(0, 1);
			}
			TestEqual(TEXT("RemoveAtSwap(Index, Count) handles shrinking correctly"), (AllowsShrinking == EAllowShrinking::Yes), (Array.Max() < 1000));
		}
		{
			Container Array;
			Array.SetNum(1000);
			Array.RemoveAllSwap([](int) -> bool
			{
				return true;
			});
			TestEqual(TEXT("RemoveAllSwap handles shrinking correctly"), (AllowsShrinking == EAllowShrinking::Yes), (Array.Max() < 1000));
		}
		{
			Container Array;
			Array.SetNumZeroed(1000);
			for (int Count=0; Count<1000; ++Count)
			{
				Array.RemoveSingleSwap(0);
			}
			TestEqual(TEXT("RemoveSingleSwap handles shrinking correctly"), (AllowsShrinking == EAllowShrinking::Yes), (Array.Max() < 1000));
		}
		{
			Container Array;
			Array.SetNumZeroed(1000);
			Array.RemoveSwap(0);
			TestEqual(TEXT("RemoveSwap handles shrinking correctly"), (AllowsShrinking == EAllowShrinking::Yes), (Array.Max() < 1000));
		}
		// NOTE: the Heap methods of TArray are currently untested.
	};

	RunArrayShrinkTests(TArray<int, FDefaultAllocator>(), EAllowShrinking::Yes);
	RunArrayShrinkTests(TArray<int, FNonshrinkingAllocator>(), EAllowShrinking::No);

	RunArrayShrinkTests(TArray<int, TInlineAllocator<32, FDefaultAllocator>>(), EAllowShrinking::Yes);
	RunArrayShrinkTests(TArray<int, TInlineAllocator<32, FNonshrinkingAllocator>>(), EAllowShrinking::No);

	return !HasAnyErrors();
}

namespace ArrayViewTests
{
	// commented out lines shouldn't compile

	struct Base
	{
		int32 b;
	};

	struct Derived : public Base
	{
		int32 d;
	};

	template<typename T>
	void TestFunction(TArrayView<T>)
	{
	}

	template<typename T>
	void TestFunction64(TArrayView64<T>)
	{
	}

	bool RunTest()
	{
		// C array + derived-to-base conversions
		Derived test1[13];
		TestFunction<Derived>(test1);
		//TestFunction<Base>(test1);
		//TestFunction<const Base>(test1);
		TestFunction64<Derived>(test1);

		// C array of pointers + derived-to-base conversions
		Derived* test2[13];
		TestFunction<const Derived* const>(test2);
		//TestFunction<const Derived*>(test2);
		TestFunction<Derived* const>(test2);
		//TestFunction<const Base* const>(test2);
		TestFunction64<const Derived* const>(test2);
		TestFunction64<Derived* const>(test2);

		// TArray + derived-to-base conversions
		TArray<Derived> test3;
		TestFunction<Derived>(test3);
		//TestFunction<Base>(test3);
		//TestFunction<const Base>(test3);
		TestFunction64<Derived>(test3);

		// const TArray
		const TArray<Base> test4;
		TestFunction<const Base>(test4);
		//TestFunction<Base>(test4);
		TestFunction64<const Base>(test4);

		// TArray of const
		TArray<const Base> test5;
		TestFunction<const Base>(test5);
		//TestFunction<Base>(test5);
		TestFunction64<const Base>(test5);

		// temporary C array
		struct Test6
		{
			Base test[13];
		};
		TestFunction<const Base>(Test6().test);
		//TestFunction<Base>(Test6().test); // shouldn't compile but VS allows it :(
		TestFunction64<const Base>(Test6().test);

		// TArrayView64 from TArrayView
		TArrayView<Derived> test7 = test1;
		TestFunction64<Derived>(test7);
		//TArrayView64<Derived> test7_64 = test1;
		//TestFunction<Derived>(test7_64);

		// TArray[64] from TArrayView[64]
		TArrayView<Derived> test8;
		TArrayView<Derived> test8_64;
		TArray<Derived> test8_32from32(test8);
		TArray<Derived> test8_32from64(test8_64);
		TArray64<Derived> test8_64from32(test8);
		TArray64<Derived> test8_64from64(test8_64);
		test8_32from32 = test8;
		test8_32from64 = test8_64;
		test8_64from32 = test8;
		test8_64from64 = test8_64;

		return true;
	}
};

UE_DISABLE_OPTIMIZATION_SHIP

FORCENOINLINE void PlaceBreakpointHere()
{
	// This function exists to support the debugger-inspecting workflow in the tests below.

	// Do some memory allocation here to ensure the compiler doesn't optimize away the function.
	void* Temp = FMemory::Malloc(16);
	FMemory::Free(Temp);
}

template <typename Resolver>
struct TSetNatvisTest
{
	template <typename Type>
	using SetType = typename Resolver::template TSetType<Type>;

	// Name       | Value | Type
	// -----------+-------+------------------------------------------------------
	// > EmptySet | Empty | TSet<int,DefaultKeyFuncs<int,0>,FDefaultSetAllocator>
	SetType<int32> EmptySet;

	// Name       | Value         | Type
	// -----------+---------------+------------------------------------------------------
	// > SlackSet | Empty, Max=16 | TSet<int,DefaultKeyFuncs<int,0>,FDefaultSetAllocator>
	SetType<int32> SlackSet;

	// Name      | Value        | Type
	// ----------+--------------+------------------------------------------------------
	// > SetVals | Num=5, Max=5 | TSet<int,DefaultKeyFuncs<int,0>,FDefaultSetAllocator>
	//     [0]   | 4            | int
	//     [1]   | 1            | int
	//     [2]   | 2            | int
	//     [3]   | 5            | int
	//     [4]   | 3            | int
	SetType<int32> SetVals = { 4, 1, 2, 5, 3 };

	// Name           | Value         | Type
	// ---------------+---------------+------------------------------------------------------
	// > SlackSetVals | Num=5, Max=16 | TSet<int,DefaultKeyFuncs<int,0>,FDefaultSetAllocator>
	//     [0]        | 4             | int
	//     [1]        | 1             | int
	//     [2]        | 2             | int
	//     [3]        | 5             | int
	//     [4]        | 3             | int
	SetType<int32> SlackSetVals = { 4, 1, 2, 5, 3 };

	// Name            | Value        | Type
	// ----------------+--------------+------------------------------------------------------
	// > SparseSetVals | Num=3, Max=5 | TSet<int,DefaultKeyFuncs<int,0>,FDefaultSetAllocator>
	//     [0]         | 4            | int
	//     [1]         | 2            | int
	//     [2]         | 3            | int
	// Note: Order different for compact set
	SetType<int32> SparseSetVals = { 4, 1, 2, 5, 3 };

	// Name      | Value        | Type
	// ----------+--------------+--------------------------------------------------------------
	// > SetStrs | Num=5, Max=5 | TSet<FString,DefaultKeyFuncs<FString,0>,FDefaultSetAllocator>
	//   > [0]   | L"d1"        | FString
	//       [0] | 100 'd'      | wchar_t
	//       [1] | 49 '1'       | wchar_t
	//   > [1]   | L"a2"        | FString
	//       [0] | 97 'a'       | wchar_t
	//       [1] | 50 '2'       | wchar_t
	//   > [2]   | L"b3"        | FString
	//       [0] | 98 'b'       | wchar_t
	//       [1] | 51 '3'       | wchar_t
	//   > [3]   | L"e4"        | FString
	//       [0] | 101 'e'      | wchar_t
	//       [1] | 52 '4'       | wchar_t
	//   > [4]   | L"c5"        | FString
	//       [0] | 99 'c'       | wchar_t
	//       [1] | 53 '5'       | wchar_t
	SetType<FString> SetStrs = { "d1", "a2", "b3", "e4", "c5" };

	// Name            | Value                | Type
	// ----------------+----------------------+--------------------------------------------------------------------------------------------------------------------------------------
	// > SetUniqueStrs | Num=5, Max=5         | TSet<TUniquePtr<FString,TDefaultDelete<FString>>,DefaultKeyFuncs<TUniquePtr<FString,TDefaultDelete<FString>>,0>,FDefaultSetAllocator>
	//   > [0]         | Ptr=0x01234567 L"d1" | TUniquePtr<FString,TDefaultDelete<FString>>
	//       [0]       | 100 'd'              | wchar_t
	//       [1]       | 49 '1'               | wchar_t
	//   > [1]         | Ptr=0x01234567 L"a2" | TUniquePtr<FString,TDefaultDelete<FString>>
	//       [0]       | 97 'a'               | wchar_t
	//       [1]       | 50 '2'               | wchar_t
	//   > [2]         | Ptr=0x01234567 L"b3" | TUniquePtr<FString,TDefaultDelete<FString>>
	//       [0]       | 98 'b'               | wchar_t
	//       [1]       | 51 '3'               | wchar_t
	//   > [3]         | Ptr=0x01234567 L"e4" | TUniquePtr<FString,TDefaultDelete<FString>>
	//       [0]       | 101 'e'              | wchar_t
	//       [1]       | 52 '4'               | wchar_t
	//   > [4]         | Ptr=0x01234567 L"c5" | TUniquePtr<FString,TDefaultDelete<FString>>
	//       [0]       | 99 'c'               | wchar_t
	//       [1]       | 53 '5'               | wchar_t
	SetType<TUniquePtr<FString>> SetUniqueStrs;

	TSetNatvisTest()
	{
		SlackSet.Reserve(16);
		SlackSetVals.Reserve(16);
		SparseSetVals.Remove(1);
		SparseSetVals.Remove(5);
		SetUniqueStrs.Reserve(5);
		SetUniqueStrs.Add(MakeUnique<FString>("d1"));
		SetUniqueStrs.Add(MakeUnique<FString>("a2"));
		SetUniqueStrs.Add(MakeUnique<FString>("b3"));
		SetUniqueStrs.Add(MakeUnique<FString>("e4"));
		SetUniqueStrs.Add(MakeUnique<FString>("c5"));
	}
};

template <typename Resolver>
struct TMapNatvisTest
{
	template <typename Key, typename Value>
	using MapType = typename Resolver::template TMapType<Key, Value>;
	
	// Name       | Value | Type
	// -----------+-------+----------------------------------------------------------------------------------
	// > EmptyMap | Empty | TMap<FString,int,FDefaultSetAllocator,TDefaultMapHashableKeyFuncs<FString,int,0>>
	MapType<FString, int32> EmptyMap;

	// Name       | Value         | Type
	// -----------+---------------+------------------------------------------------------
	// > SlackMap | Empty, Max=16 | TMap<FString,int,FDefaultSetAllocator,TDefaultMapHashableKeyFuncs<FString,int,0>>
	MapType<FString, int32> SlackMap;

	// Name       | Value        | Type
	// -----------+--------------+------------------------------------------------------
	// > MapVals  | Num=5, Max=5 | TMap<FString,int,FDefaultSetAllocator,TDefaultMapHashableKeyFuncs<FString,int,0>>
	//     [L"d"] | 1            | int
	//     [L"a"] | 2            | int
	//     [L"b"] | 3            | int
	//     [L"e"] | 4            | int
	//     [L"c"] | 5            | int
	MapType<FString, int32> MapVals = { { "d", 1 }, { "a", 2 }, { "b", 3 }, { "e", 4 }, { "c", 5 } };

	// Name           | Value         | Type
	// ---------------+---------------+------------------------------------------------------
	// > SlackMapVals | Num=5, Max=16 | TMap<FString,int,FDefaultSetAllocator,TDefaultMapHashableKeyFuncs<FString,int,0>>
	//     [L"d"]     | 1             | int
	//     [L"a"]     | 2             | int
	//     [L"b"]     | 3             | int
	//     [L"e"]     | 4             | int
	//     [L"c"]     | 5             | int
	MapType<FString, int32> SlackMapVals = { { "d", 1 }, { "a", 2 }, { "b", 3 }, { "e", 4 }, { "c", 5 } };

	// Name            | Value        | Type
	// ----------------+--------------+------------------------------------------------------
	// > SparseMapVals | Num=3, Max=5 | TMap<FString,int,FDefaultSetAllocator,TDefaultMapHashableKeyFuncs<FString,int,0>>
	//     [L"d"]      | 1            | int
	//     [L"b"]      | 3            | int
	//     [L"c"]      | 5            | int
	// Note: Order different for compact map
	MapType<FString, int32> SparseMapVals = { { "d", 1 }, { "a", 2 }, { "b", 3 }, { "e", 4 }, { "c", 5 } };

	// Name                      | Value        | Type
	// --------------------------+--------------+----------------------------------------------------------------------------------------------------------------------------------------------------------
	// > MapUniqueVals           | Num=5, Max=5 | TMap<TUniquePtr<FString,TDefaultDelete<FString>>,int,FDefaultSetAllocator,TDefaultMapHashableKeyFuncs<TUniquePtr<FString,TDefaultDelete<FString>>,int,0>>
	//     [Ptr=0x12345678 L"d"] | 1            | int
	//     [Ptr=0x12345678 L"a"] | 2            | int
	//     [Ptr=0x12345678 L"b"] | 3            | int
	//     [Ptr=0x12345678 L"e"] | 4            | int
	//     [Ptr=0x12345678 L"c"] | 5            | int
	MapType<TUniquePtr<FString>, int32> MapUniqueVals;

	TMapNatvisTest()
	{
		SlackMap.Reserve(16);
		SlackMapVals.Reserve(16);
		SparseMapVals.Remove("a");
		SparseMapVals.Remove("e");
		MapUniqueVals.Reserve(5);
		MapUniqueVals.Emplace(MakeUnique<FString>("d"), 1);
		MapUniqueVals.Emplace(MakeUnique<FString>("a"), 2);
		MapUniqueVals.Emplace(MakeUnique<FString>("b"), 3);
		MapUniqueVals.Emplace(MakeUnique<FString>("e"), 4);
		MapUniqueVals.Emplace(MakeUnique<FString>("c"), 5);
	}
};

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FContainersNatvisTest, "System.Core.Containers.Natvis", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool FContainersNatvisTest::RunTest(const FString& Parameters)
{
	// This results of this test are intended to be inspected visually in a platform's debugger, to ensure that the .natvis is correct.
	// A breakpoint should be placed on the last line of each block, the block variables observed in the debugger and compared with the
	// textual description.

	// TArray
	{
		// Name       | Value | Type
		// -----------+-------+---------------------------------------
		// > EmptyArr | Empty | TArray<int,TSizedDefaultAllocator<32>>
		TArray<int32> EmptyArr;

		// Name       | Value         | Type
		// -----------+---------------+---------------------------------------
		// > SlackArr | Empty, Max=16 | TArray<int,TSizedDefaultAllocator<32>>
		TArray<int32> SlackArr;
		SlackArr.Reserve(16);

		// Name      | Value        | Type
		// ----------+--------------+---------------------------------------
		// > ArrVals | Num=5, Max=5 | TArray<int,TSizedDefaultAllocator<32>>
		//     [0]   | 4            | int
		//     [1]   | 1            | int
		//     [2]   | 2            | int
		//     [3]   | 5            | int
		//     [4]   | 3            | int
		TArray<int32> ArrVals = { 4, 1, 2, 5, 3 };

		// Name           | Value        | Type
		// ---------------+--------------+---------------------------------------
		// > SlackArrVals | Num=3, Max=5 | TArray<int,TSizedDefaultAllocator<32>>
		//     [0]        | 4            | int
		//     [1]        | 2            | int
		//     [2]        | 3            | int
		TArray<int32> SlackArrVals = { 4, 1, 2, 5, 3 };
		SlackArrVals.RemoveAt(3);
		SlackArrVals.RemoveAt(1);

		// Name      | Value        | Type
		// ----------+--------------+-------------------------------------------
		// > ArrStrs | Num=5, Max=5 | TArray<FString,TSizedDefaultAllocator<32>>
		//   > [0]   | L"d1"        | FString
		//       [0] | 100 'd'      | wchar_t
		//       [1] | 49 '1'       | wchar_t
		//   > [1]   | L"a2"        | FString
		//       [0] | 97 'a'       | wchar_t
		//       [1] | 50 '2'       | wchar_t
		//   > [2]   | L"b3"        | FString
		//       [0] | 98 'b'       | wchar_t
		//       [1] | 51 '3'       | wchar_t
		//   > [3]   | L"e4"        | FString
		//       [0] | 101 'e'      | wchar_t
		//       [1] | 52 '4'       | wchar_t
		//   > [4]   | L"c5"        | FString
		//       [0] | 99 'c'       | wchar_t
		//       [1] | 53 '5'       | wchar_t
		TArray<FString> ArrStrs = { "d1", "a2", "b3", "e4", "c5" };

		// Name            | Value                | Type
		// ----------------+----------------------+-------------------------------------------------------------------------------
		// > ArrUniqueStrs | Num=5, Max=5         | TArray<TUniquePtr<FString,TDefaultDelete<FString>>,TSizedDefaultAllocator<32>>
		//   > [0]         | Ptr=0x01234567 L"d1" | TUniquePtr<FString,TDefaultDelete<FString>>
		//       [0]       | 100 'd'              | wchar_t
		//       [1]       | 49 '1'               | wchar_t
		//   > [1]         | Ptr=0x01234567 L"a2" | TUniquePtr<FString,TDefaultDelete<FString>>
		//       [0]       | 97 'a'               | wchar_t
		//       [1]       | 50 '2'               | wchar_t
		//   > [2]         | Ptr=0x01234567 L"b3" | TUniquePtr<FString,TDefaultDelete<FString>>
		//       [0]       | 98 'b'               | wchar_t
		//       [1]       | 51 '3'               | wchar_t
		//   > [3]         | Ptr=0x01234567 L"e4" | TUniquePtr<FString,TDefaultDelete<FString>>
		//       [0]       | 101 'e'              | wchar_t
		//       [1]       | 52 '4'               | wchar_t
		//   > [4]         | Ptr=0x01234567 L"c5" | TUniquePtr<FString,TDefaultDelete<FString>>
		//       [0]       | 99 'c'               | wchar_t
		//       [1]       | 53 '5'               | wchar_t
		TArray<TUniquePtr<FString>> ArrUniqueStrs;
		ArrUniqueStrs.Reserve(5);
		ArrUniqueStrs.Add(MakeUnique<FString>("d1"));
		ArrUniqueStrs.Add(MakeUnique<FString>("a2"));
		ArrUniqueStrs.Add(MakeUnique<FString>("b3"));
		ArrUniqueStrs.Add(MakeUnique<FString>("e4"));
		ArrUniqueStrs.Add(MakeUnique<FString>("c5"));

		PlaceBreakpointHere();
	}

	{
		// TSet
		TSetNatvisTest<FSetResolver> SetNatvisTest;
		TSetNatvisTest<FSparseSetResolver> SparseSetNatvisTest;
		TSetNatvisTest<FCompactSetResolver> CompactNatvisTest;

		// TMap
		TMapNatvisTest<FSetResolver> MapNatvisTest;
		TMapNatvisTest<FSparseSetResolver> SparseMapNatvisTest;
		TMapNatvisTest<FCompactSetResolver> CompactMapNatvisTest;

		PlaceBreakpointHere();
	}

	// TSortedSet
	{
		// Name        | Value | Type
		// ------------+-------+------------------------------------------------------
		// > EmptySSet | Empty | TSortedSet<int,TSizedDefaultAllocator<32>,TLess<int>>
		TSortedSet<int32> EmptySSet;

		// Name        | Value         | Type
		// ------------+---------------+------------------------------------------------------
		// > SlackSSet | Empty, Max=16 | TSortedSet<int,TSizedDefaultAllocator<32>,TLess<int>>
		TSortedSet<int32> SlackSSet;
		SlackSSet.Reserve(16);

		// Name       | Value        | Type
		// -----------+--------------+------------------------------------------------------
		// > SSetVals | Num=5, Max=5 | TSortedSet<int,TSizedDefaultAllocator<32>,TLess<int>>
		//     [0]    | 1            | int
		//     [1]    | 2            | int
		//     [2]    | 3            | int
		//     [3]    | 4            | int
		//     [4]    | 5            | int
		TSortedSet<int32> SSetVals = { 4, 1, 2, 5, 3 };

		// Name           | Value         | Type
		// ---------------+---------------+------------------------------------------------------
		// > SlackSetVals | Num=5, Max=16 | TSortedSet<int,TSizedDefaultAllocator<32>,TLess<int>>
		//     [0]        | 1             | int
		//     [1]        | 2             | int
		//     [2]        | 3             | int
		//     [3]        | 4             | int
		//     [4]        | 5             | int
		TSortedSet<int32> SlackSSetVals = { 4, 1, 2, 5, 3 };
		SlackSSetVals.Reserve(16);

		// Name             | Value        | Type
		// -----------------+--------------+------------------------------------------------------
		// > SparseSSetVals | Num=3, Max=5 | TSortedSet<int,TSizedDefaultAllocator<32>,TLess<int>>
		//     [0]          | 2            | int
		//     [1]          | 3            | int
		//     [2]          | 4            | int
		TSortedSet<int32> SparseSSetVals = { 4, 1, 2, 5, 3 };
		SparseSSetVals.Remove(1);
		SparseSSetVals.Remove(5);

		// Name       | Value        | Type
		// -----------+--------------+--------------------------------------------------------------
		// > SSetStrs | Num=5, Max=5 | TSortedSet<FString,TSizedDefaultAllocator<32>,TLess<FString>>
		//   > [0]    | L"a2"        | FString
		//       [0]  | 97 'a'       | wchar_t
		//       [1]  | 50 '2'       | wchar_t
		//   > [1]    | L"b3"        | FString
		//       [0]  | 98 'b'       | wchar_t
		//       [1]  | 51 '3'       | wchar_t
		//   > [2]    | L"c5"        | FString
		//       [0]  | 99 'c'       | wchar_t
		//       [1]  | 53 '5'       | wchar_t
		//   > [3]    | L"d1"        | FString
		//       [0]  | 100 'd'      | wchar_t
		//       [1]  | 49 '1'       | wchar_t
		//   > [4]    | L"e4"        | FString
		//       [0]  | 101 'e'      | wchar_t
		//       [1]  | 52 '4'       | wchar_t
		TSortedSet<FString> SSetStrs = { "d1", "a2", "b3", "e4", "c5" };

		// Name             | Value                | Type
		// -----------------+----------------------+--------------------------------------------------------------------------------------------------------------------------------------
		// > SSetUniqueStrs | Num=5, Max=5         | TSortedSet<TUniquePtr<FString,TDefaultDelete<FString>>,TSizedDefaultAllocator<32>,TLess<TUniquePtr<FString,TDefaultDelete<FString>>>>
		//   > [0]          | Ptr=0x01234567 L"a2" | TUniquePtr<FString,TDefaultDelete<FString>>
		//       [0]        | 97 'a'               | wchar_t
		//       [1]        | 50 '2'               | wchar_t
		//   > [1]          | Ptr=0x01234567 L"b3" | TUniquePtr<FString,TDefaultDelete<FString>>
		//       [0]        | 98 'b'               | wchar_t
		//       [1]        | 51 '3'               | wchar_t
		//   > [2]          | Ptr=0x01234567 L"c5" | TUniquePtr<FString,TDefaultDelete<FString>>
		//       [0]        | 99 'c'               | wchar_t
		//       [1]        | 53 '5'               | wchar_t
		//   > [3]          | Ptr=0x01234567 L"d1" | TUniquePtr<FString,TDefaultDelete<FString>>
		//       [0]        | 100 'd'              | wchar_t
		//       [1]        | 49 '1'               | wchar_t
		//   > [4]          | Ptr=0x01234567 L"e4" | TUniquePtr<FString,TDefaultDelete<FString>>
		//       [0]        | 101 'e'              | wchar_t
		//       [1]        | 52 '4'               | wchar_t
		TSortedSet<TUniquePtr<FString>> SSetUniqueStrs;
		SSetUniqueStrs.Reserve(5);
		SSetUniqueStrs.Add(MakeUnique<FString>("d1"));
		SSetUniqueStrs.Add(MakeUnique<FString>("a2"));
		SSetUniqueStrs.Add(MakeUnique<FString>("b3"));
		SSetUniqueStrs.Add(MakeUnique<FString>("e4"));
		SSetUniqueStrs.Add(MakeUnique<FString>("c5"));

		PlaceBreakpointHere();
	}

	// TSortedMap
	{
		// Name        | Value | Type
		// ------------+-------+--------------------------------------------------------------------------
		// > EmptySMap | Empty | TSortedMap<FString,int,TSizedDefaultAllocator<32>,TLess<const FString &>>
		TSortedMap<FString, int32> EmptySMap;

		// Name        | Value         | Type
		// ------------+---------------+------------------------------------------------------
		// > SlackSMap | Empty, Max=16 | TSortedMap<FString,int,TSizedDefaultAllocator<32>,TLess<const FString &>>
		TSortedMap<FString, int32> SlackSMap;
		SlackSMap.Reserve(16);

		// Name       | Value        | Type
		// -----------+--------------+------------------------------------------------------
		// > SMapVals | Num=5, Max=5 | TSortedMap<FString,int,TSizedDefaultAllocator<32>,TLess<const FString &>>
		//     [L"a"] | 2            | int
		//     [L"b"] | 3            | int
		//     [L"c"] | 5            | int
		//     [L"d"] | 1            | int
		//     [L"e"] | 4            | int
		TSortedMap<FString, int32> SMapVals = { { "d", 1 }, { "a", 2 }, { "b", 3 }, { "e", 4 }, { "c", 5 } };

		// Name            | Value         | Type
		// ----------------+---------------+------------------------------------------------------
		// > SlackSMapVals | Num=5, Max=16 | TSortedMap<FString,int,TSizedDefaultAllocator<32>,TLess<const FString &>>
		//     [L"a"]      | 2             | int
		//     [L"b"]      | 3             | int
		//     [L"c"]      | 5             | int
		//     [L"d"]      | 1             | int
		//     [L"e"]      | 4             | int
		TSortedMap<FString, int32> SlackSMapVals = { { "d", 1 }, { "a", 2 }, { "b", 3 }, { "e", 4 }, { "c", 5 } };
		SlackSMapVals.Reserve(16);

		// Name             | Value        | Type
		// -----------------+--------------+------------------------------------------------------
		// > SparseSMapVals | Num=3, Max=5 | TSortedMap<FString,int,TSizedDefaultAllocator<32>,TLess<const FString &>>
		//     [L"b"]       | 3            | int
		//     [L"c"]       | 5            | int
		//     [L"d"]       | 1            | int
		TSortedMap<FString, int32> SparseSMapVals = { { "d", 1 }, { "a", 2 }, { "b", 3 }, { "e", 4 }, { "c", 5 } };
		SparseSMapVals.Remove("a");
		SparseSMapVals.Remove("e");

		// Name                      | Value        | Type
		// --------------------------+--------------+--------------------------------------------------------------------------------------------------------------------------------------------------
		// > SMapUniqueVals          | Num=5, Max=5 | TSortedMap<TUniquePtr<FString,TDefaultDelete<FString>>,int,TSizedDefaultAllocator<32>,TLess<const TUniquePtr<FString,TDefaultDelete<FString>> &>>
		//     [Ptr=0x12345678 L"a"] | 2            | int
		//     [Ptr=0x12345678 L"b"] | 3            | int
		//     [Ptr=0x12345678 L"c"] | 5            | int
		//     [Ptr=0x12345678 L"d"] | 1            | int
		//     [Ptr=0x12345678 L"e"] | 4            | int
		TSortedMap<TUniquePtr<FString>, int32> SMapUniqueVals;
		SMapUniqueVals.Reserve(5);
		SMapUniqueVals.Emplace(MakeUnique<FString>("d"), 1);
		SMapUniqueVals.Emplace(MakeUnique<FString>("a"), 2);
		SMapUniqueVals.Emplace(MakeUnique<FString>("b"), 3);
		SMapUniqueVals.Emplace(MakeUnique<FString>("e"), 4);
		SMapUniqueVals.Emplace(MakeUnique<FString>("c"), 5);

		PlaceBreakpointHere();
	}

	return true;
}

UE_ENABLE_OPTIMIZATION_SHIP

#endif // WITH_DEV_AUTOMATION_TESTS
