// Copyright Epic Games, Inc. All Rights Reserved.

#include "Containers/DirectoryTree.h"

#include "Algo/BinarySearch.h"
#include "Algo/Sort.h"
#include "Misc/AutomationTest.h"

namespace UE::DirectoryTree
{

void FixupPathSeparator(FStringBuilderBase& InOutPath, int32 StartIndex, TCHAR InPathSeparator)
{
	if (InPathSeparator == '/')
	{
		return;
	}
	int32 SeparatorIndex;
	while (InOutPath.ToView().RightChop(StartIndex).FindChar('/', SeparatorIndex))
	{
		StartIndex += SeparatorIndex;
		InOutPath.GetData()[StartIndex] = InPathSeparator;
	}
}

int32 FindInsertionIndex(int32 NumChildNodes, const TUniquePtr<FString[]>& RelPaths, FStringView FirstPathComponent, bool& bOutExists)
{
	TConstArrayView<FString> RelPathsRange(RelPaths.Get(), NumChildNodes);
	int32 Index = Algo::LowerBound(RelPathsRange, FirstPathComponent,
		[](const FString& ChildRelPath, FStringView FirstPathComponent)
		{
			return FPathViews::Less(ChildRelPath, FirstPathComponent);
		}
	);
	bOutExists = Index < NumChildNodes && FPathViews::IsParentPathOf(FirstPathComponent, RelPaths[Index]);
	return Index;
}

} // namespace UE::DirectoryTree
