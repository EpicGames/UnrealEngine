// Copyright Epic Games, Inc. All Rights Reserved.

#include "Containers/HashTable.h"

CORE_API uint32 FHashTable::EmptyHash[1] = { ~0u };

CORE_API void FHashTable::Resize( uint32 NewIndexSize )
{
	if( NewIndexSize == IndexSize )
	{
		return;
	}

	if( NewIndexSize == 0 )
	{
		Free();
		return;
	}

	if( IndexSize == 0 )
	{
		HashMask = (uint16)(HashSize - 1);
		Hash = new uint32[ HashSize ];
		FMemory::Memset( Hash, 0xff, HashSize * 4 );
	}

	uint32* NewNextIndex = new uint32[ NewIndexSize ];

	if( NextIndex )
	{
		if(NewIndexSize < IndexSize)
		{
			// If the new size is less than the old size then the linked list of indices could have been invalidated
			// Collapse gaps in the resulting lists for each hash
			for(uint32 ListIndex = 0; ListIndex < HashSize; ++ListIndex)
			{
				uint32* Node = &Hash[ListIndex];

				while(IsValid(*Node))
				{
					// For any list node >= NewIndex size: Next = Next->Next
					if(*Node >= NewIndexSize)
					{
						*Node = NextIndex[*Node];

						continue;
					}

					Node = &NextIndex[*Node];
				}
			}
		}

		FMemory::Memcpy( NewNextIndex, NextIndex, FMath::Min(IndexSize, NewIndexSize) * 4 );

		delete[] NextIndex;
	}
	
	IndexSize = NewIndexSize;
	NextIndex = NewNextIndex;
}

SIZE_T FHashTable::GetAllocatedSize() const
{
	return
		sizeof(EmptyHash) +
		(sizeof(uint32*) * 2) +
		(sizeof(uint32) * (3 + IndexSize + HashSize));
}

CORE_API float FHashTable::AverageSearch() const
{
	uint32 SumAvgSearch = 0;
	uint32 NumElements = 0;
	for( uint32 Key = 0; Key < HashSize; Key++ )
	{
		uint32 NumInBucket = 0;
		for( uint32 i = First( (uint16)Key ); IsValid( i ); i = Next( i ) )
		{
			NumInBucket++;
		}

		SumAvgSearch += NumInBucket * ( NumInBucket + 1 );
		NumElements  += NumInBucket;
	}
	return (float)( SumAvgSearch >> 1 ) / (float)NumElements;
}
