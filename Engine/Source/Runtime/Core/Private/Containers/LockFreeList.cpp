// Copyright Epic Games, Inc. All Rights Reserved.

#include "Containers/LockFreeList.h"
#include "AutoRTFM.h"
#include "HAL/PlatformProcess.h"
#include "HAL/IConsoleManager.h"
#include "Stats/Stats.h"



DEFINE_LOG_CATEGORY(LogLockFreeList);

DECLARE_MEMORY_STAT(TEXT("Lock Free List Links"), STAT_LockFreeListLinks, STATGROUP_Memory);


#if !UE_BUILD_SHIPPING && !UE_BUILD_TEST

void DoTestCriticalStall()
{
	float Test = FMath::FRand();
	if (Test < .001)
	{
		FPlatformProcess::SleepNoStats(0.001f);
	}
	else if (Test < .01f)
	{
		FPlatformProcess::SleepNoStats(0.0f);
	}
}

int32 GTestCriticalStalls = 0;
static FAutoConsoleVariableRef CVarTestCriticalLockFree(
	TEXT("TaskGraph.TestCriticalLockFree"),
	GTestCriticalStalls,
	TEXT("If > 0, then we sleep periodically at critical points in the lock free lists. Threads must not starve...this will encourage them to starve at the right place to find livelocks."),
	ECVF_Cheat
	);

#endif

void LockFreeTagCounterHasOverflowed()
{
	UE_LOGF(LogTemp, Log, "LockFree Tag has overflowed...(not a problem).");
	FPlatformProcess::Sleep(.001f);
}

void LockFreeLinksExhausted(uint32 TotalNum)
{
	UE_LOGF(LogTemp, Fatal, "Consumed %d lock free links; there are no more.", TotalNum);
}

static void ChangeMem(int64 Delta)
{
#if 0   // this is not safe; we cannot change stats in the middle of a lock free operation
	static FThreadSafeCounter LockFreeListMem;
	LockFreeListMem.Add(Delta);
	if (GIsRunning)
	{
		SET_MEMORY_STAT(STAT_LockFreeListLinks, LockFreeListMem.GetValue());
	}
#endif
}

void* LockFreeAllocLinks(SIZE_T AllocSize)
{
	ChangeMem(AllocSize);
	return FMemory::Malloc(AllocSize);
}
void LockFreeFreeLinks(SIZE_T AllocSize, void* Ptr)
{
	ChangeMem(-int32(AllocSize));
	FMemory::Free(Ptr);
}

class LockFreeLinkAllocator_TLSCache : public FNoncopyable
{
	enum
	{
		NUM_PER_BUNDLE = 64,
	};

	typedef FLockFreeLinkPolicy::TLink TLink;
	typedef FLockFreeLinkPolicy::TLinkPtr TLinkPtr;

public:

	LockFreeLinkAllocator_TLSCache()
	{
		checkf(!AutoRTFM::IsClosed() || !AutoRTFM::IsOnCurrentTransactionStack(this), TEXT("Not allowed to construct a stack local within a transaction."));

		UE_AUTORTFM_OPEN
		{
			check(IsInGameThread());
			TlsSlot = FPlatformTLS::AllocTlsSlot();
			check(FPlatformTLS::IsValidTlsSlot(TlsSlot));
		};
		AutoRTFM::PushOnAbortHandler(this, [this]
		{
			FPlatformTLS::FreeTlsSlot(TlsSlot);
			TlsSlot = FPlatformTLS::InvalidTlsSlot;
		});
	}
	
	/** Destructor, leaks all of the memory **/
	~LockFreeLinkAllocator_TLSCache()
	{
		AutoRTFM::PopAllOnAbortHandlers(this);
		UE_AUTORTFM_ONCOMMIT(this)
		{
			FPlatformTLS::FreeTlsSlot(TlsSlot);
			TlsSlot = FPlatformTLS::InvalidTlsSlot;
		};
	}

	/**
	* Allocates a memory block of size SIZE.
	*
	* @return Pointer to the allocated memory.
	* @see Free
	*/
	TLinkPtr Pop()
	{
		if (AutoRTFM::IsClosed())
		{
			TLinkPtr Ptr = AutoRTFM::Open([&]
			{
				return Pop();
			});
			AutoRTFM::PushOnAbortHandler(this, [this, Ptr]
			{
				Push(Ptr);
			});
			return Ptr;
		}

		FThreadLocalCache& TLS = GetTLS();

		if (!TLS.PartialBundle)
		{
			if (TLS.FullBundle)
			{
				TLS.PartialBundle = TLS.FullBundle;
				TLS.FullBundle = 0;
			}
			else
			{
				TLS.PartialBundle = GlobalFreeListBundles.Pop();
				if (!TLS.PartialBundle)
				{
					int32 FirstIndex = FLockFreeLinkPolicy::LinkAllocator.Alloc(NUM_PER_BUNDLE);
					for (int32 Index = 0; Index < NUM_PER_BUNDLE; Index++)
					{
						TLink* Event = FLockFreeLinkPolicy::IndexToLink(FirstIndex + Index);
						Event->DoubleNext.Init();
						Event->SingleNext.store(0, std::memory_order_relaxed);
						Event->Payload.store((void*)UPTRINT(TLS.PartialBundle), std::memory_order_relaxed);
						TLS.PartialBundle = FLockFreeLinkPolicy::IndexToPtr(FirstIndex + Index);
					}
				}
			}
			TLS.NumPartial = NUM_PER_BUNDLE;
		}
		TLinkPtr Result = TLS.PartialBundle;
		TLink* ResultP = FLockFreeLinkPolicy::DerefLink(TLS.PartialBundle);
		TLS.PartialBundle = TLinkPtr(UPTRINT(ResultP->Payload.load(std::memory_order_relaxed)));
		TLS.NumPartial--;
		//checkLockFreePointerList(TLS.NumPartial >= 0 && ((!!TLS.NumPartial) == (!!TLS.PartialBundle)));
		ResultP->Payload.store(nullptr, std::memory_order_relaxed);
		checkLockFreePointerList(!ResultP->DoubleNext.GetPtr() && !ResultP->SingleNext.load(std::memory_order_relaxed));
		return Result;
	}

	/**
	* Puts a memory block previously obtained from Allocate() back on the free list for future use.
	*
	* @param Item The item to free.
	* @see Allocate
	*/
	void Push(TLinkPtr Item)
	{
		if (AutoRTFM::IsClosed())
		{
			UE_AUTORTFM_ONCOMMIT(this, Item)
			{
				return Push(Item);
			};
		}
		else
		{
			FThreadLocalCache& TLS = GetTLS();
			if (TLS.NumPartial >= NUM_PER_BUNDLE)
			{
				if (TLS.FullBundle)
				{
					GlobalFreeListBundles.Push(TLS.FullBundle);
					//TLS.FullBundle = nullptr;
				}
				TLS.FullBundle = TLS.PartialBundle;
				TLS.PartialBundle = 0;
				TLS.NumPartial = 0;
			}
			TLink* ItemP = FLockFreeLinkPolicy::DerefLink(Item);
			ItemP->DoubleNext.SetPtr(0);
			ItemP->SingleNext.store(0, std::memory_order_relaxed);
			ItemP->Payload.store((void*)UPTRINT(TLS.PartialBundle), std::memory_order_relaxed);
			TLS.PartialBundle = Item;
			TLS.NumPartial++;
		}
	}

private:

	/** struct for the TLS cache. */
	struct FThreadLocalCache
	{
		TLinkPtr FullBundle;
		TLinkPtr PartialBundle;
		int32 NumPartial;

		FThreadLocalCache()
			: FullBundle(0)
			, PartialBundle(0)
			, NumPartial(0)
		{
		}
	};

	FThreadLocalCache& GetTLS()
	{
		checkSlow(FPlatformTLS::IsValidTlsSlot(TlsSlot));
		FThreadLocalCache* TLS = (FThreadLocalCache*)FPlatformTLS::GetTlsValue(TlsSlot);
		if (!TLS)
		{
			TLS = new FThreadLocalCache();
			FPlatformTLS::SetTlsValue(TlsSlot, TLS);
		}
		return *TLS;
	}

	/** Slot for TLS struct. */
	uint32 TlsSlot;

	/** Lock free list of free memory blocks, these are all linked into a bundle of NUM_PER_BUNDLE. */
	FLockFreePointerListLIFORoot<PLATFORM_CACHE_LINE_SIZE> GlobalFreeListBundles;
};

static LockFreeLinkAllocator_TLSCache& GetLockFreeAllocator()
{
	// make memory that will not go away, a replacement for TLazySingleton, which will still get destructed
	alignas(LockFreeLinkAllocator_TLSCache) static unsigned char Data[sizeof(LockFreeLinkAllocator_TLSCache)];
	static bool bIsInitialized = false;
	if (!bIsInitialized)
	{
		::new((void*)Data)LockFreeLinkAllocator_TLSCache();
		bIsInitialized = true;
	}
	return *(LockFreeLinkAllocator_TLSCache*)Data;
}

void FLockFreeLinkPolicy::FreeLockFreeLink(FLockFreeLinkPolicy::TLinkPtr Item)
{
	GetLockFreeAllocator().Push(Item);
}

FLockFreeLinkPolicy::TLinkPtr FLockFreeLinkPolicy::AllocLockFreeLink()
{
	FLockFreeLinkPolicy::TLinkPtr Result = GetLockFreeAllocator().Pop();
	// this can only really be a mem stomp
	checkLockFreePointerList(Result && !FLockFreeLinkPolicy::DerefLink(Result)->DoubleNext.GetPtr() && !FLockFreeLinkPolicy::DerefLink(Result)->Payload.load(std::memory_order_relaxed) && !FLockFreeLinkPolicy::DerefLink(Result)->SingleNext.load(std::memory_order_relaxed));
	return Result;
}

FLockFreeLinkPolicy::TAllocator FLockFreeLinkPolicy::LinkAllocator;
