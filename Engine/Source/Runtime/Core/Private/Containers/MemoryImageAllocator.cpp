// Copyright Epic Games, Inc. All Rights Reserved.

#include "Containers/ResourceArray.h"
#include "Containers/StringFwd.h"
#include "Misc/StringBuilder.h"
#include "Serialization/MemoryImage.h"
#include "Serialization/MemoryImageWriter.h"
#include "Serialization/MemoryLayout.h"

IMPLEMENT_ABSTRACT_TYPE_LAYOUT(FResourceArrayInterface);

namespace UE::Core::Private
{
	[[noreturn]] FORCENOINLINE void OnInvalidMemoryImageAllocatorNum(int32 NewNum, SIZE_T NumBytesPerElement)
	{
		UE_LOG(LogCore, Fatal, TEXT("Trying to resize TMemoryImageAllocator to an invalid size of %d with element size %" SIZE_T_FMT), NewNum, NumBytesPerElement);
		for (;;);
	}
}

FMemoryImageAllocatorBase::~FMemoryImageAllocatorBase()
{
	if (!Data.IsFrozen())
	{
		FScriptContainerElement* LocalData = Data.Get();
		if (LocalData)
		{
			FMemory::Free(LocalData);
		}
	}
}

void FMemoryImageAllocatorBase::MoveToEmpty(FMemoryImageAllocatorBase& Other)
{
	checkSlow(this != &Other);
	if (!Data.IsFrozen())
	{
		FScriptContainerElement* LocalData = Data.Get();
		if (LocalData)
		{
			FMemory::Free(LocalData);
		}
	}
	Data = Other.Data;
	Other.Data = nullptr;
}

void FMemoryImageAllocatorBase::ResizeAllocation(int32 CurrentNum, int32 NewMax, SIZE_T NumBytesPerElement, uint32 Alignment)
{
	FScriptContainerElement* LocalData = Data.Get();
	// Avoid calling FMemory::Realloc( nullptr, 0 ) as ANSI C mandates returning a valid pointer which is not what we want.
	if (Data.IsFrozen())
	{
		// Can't grow a frozen array or shrink below zero
		if (UNLIKELY((uint32)NewMax > (uint32)CurrentNum))
		{
			UE::Core::Private::OnInvalidMemoryImageAllocatorNum(NewMax, NumBytesPerElement);
		}
	}
	else if (LocalData || NewMax > 0)
	{
		static_assert(sizeof(int32) <= sizeof(SIZE_T), "SIZE_T is expected to be larger than int32");

		// Check for under/overflow
		if (UNLIKELY(NewMax < 0 || NumBytesPerElement < 1 || NumBytesPerElement > (SIZE_T)MAX_int32))
		{
			UE::Core::Private::OnInvalidMemoryImageAllocatorNum(NewMax, NumBytesPerElement);
		}

		Data = (FScriptContainerElement*)FMemory::Realloc(LocalData, NewMax*NumBytesPerElement, Alignment);
	}
}

void FMemoryImageAllocatorBase::WriteMemoryImage(FMemoryImageWriter& Writer, const FTypeLayoutDesc& TypeDesc, int32 NumAllocatedElements, uint32 Alignment) const
{
	if (NumAllocatedElements > 0)
	{
		const void* RawPtr = GetAllocation();
		check(RawPtr);
		FMemoryImageWriter ArrayWriter = Writer.WritePointer(TypeDesc);
		ArrayWriter.WriteAlignment(Alignment);
		ArrayWriter.WriteObjectArray(RawPtr, TypeDesc, NumAllocatedElements);
	}
	else
	{
		Writer.WriteNullPointer();
	}
}

void FMemoryImageAllocatorBase::CopyUnfrozen(const FMemoryUnfreezeContent& Context, const FTypeLayoutDesc& TypeDesc, int32 NumAllocatedElements, void* OutDst) const
{
	if (NumAllocatedElements > 0)
	{
		const uint8* FrozenObject = (uint8*)GetAllocation();
		uint8* UnfrozenDst = (uint8*)OutDst;

		FTypeLayoutDesc::FUnfrozenCopyFunc* Func = TypeDesc.UnfrozenCopyFunc;
		const uint32 ElementSize = TypeDesc.Size;
		const uint32 ElementAlign = TypeDesc.Alignment;
		const uint32 FrozenElementAlign = Freeze::GetTargetAlignment(TypeDesc, Context.FrozenLayoutParameters);
		uint32 FrozenOffset = 0u;
		uint32 UnfrozenOffset = 0u;

		for (int32 i = 0; i < NumAllocatedElements; ++i)
		{
			const uint32 FrozenElementSize = Func(Context, FrozenObject + FrozenOffset, TypeDesc, UnfrozenDst + UnfrozenOffset);
			FrozenOffset = Align(FrozenOffset + FrozenElementSize, FrozenElementAlign);
			UnfrozenOffset = Align(UnfrozenOffset + ElementSize, ElementAlign);
		}
	}
}

void FMemoryImageAllocatorBase::ToString(const FTypeLayoutDesc& TypeDesc, int32 NumAllocatedElements, int32 MaxAllocatedElements, const FPlatformTypeLayoutParameters& LayoutParams, FMemoryToStringContext& OutContext) const
{
	OutContext.String->Appendf(TEXT("TArray<%s>, Num: %d, Max: %d\n"), TypeDesc.Name, NumAllocatedElements, MaxAllocatedElements);
	++OutContext.Indent;

	const void* RawPtr = GetAllocation();
	FTypeLayoutDesc::FToStringFunc* Func = TypeDesc.ToStringFunc;
	const uint32 ElementSize = TypeDesc.Size;

	for (int32 i = 0; i < NumAllocatedElements; ++i)
	{
		OutContext.AppendIndent();
		OutContext.String->Appendf(TEXT("[%d]: "), i);
		Func((uint8*)RawPtr + ElementSize * i,
			TypeDesc,
			LayoutParams,
			OutContext);
	}

	--OutContext.Indent;
}
