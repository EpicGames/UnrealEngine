// Copyright Epic Games, Inc. All Rights Reserved.

#include "Containers/StackTracker.h"

#include "Containers/UnrealString.h"
#include "CoreGlobals.h"
#include "HAL/PlatformMemory.h"
#include "HAL/PlatformStackWalk.h"
#include "HAL/UnrealMemory.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CString.h"
#include "Misc/Crc.h"
#include "Misc/OutputDevice.h"
#include "Templates/UnrealTemplate.h"


DEFINE_LOG_CATEGORY_STATIC(LogStackTracker, Log, All);

/** Constructor, initializing all member variables */
FStackTracker::FStackTracker(StackTrackerUpdateFn InUpdateFn, StackTrackerReportFn InReportFn, StackTrackerDeleteUserDataFn InDeleteUserDataFn, bool bInIsEnabled)
	: bAvoidCapturing(false)
	, bIsEnabled(bInIsEnabled)
	, StartFrameCounter(0)
	, StopFrameCounter(0)
	, UpdateFn(InUpdateFn)
	, ReportFn(InReportFn)
	, DeleteUserDataFn(InDeleteUserDataFn ? InDeleteUserDataFn : &FStackTracker::DefaultDeleteUserDataFn)
{

}

void FStackTracker::DefaultDeleteUserDataFn(void* UserData)
{
	FMemory::Free(UserData);
}

/**
 * Captures the current stack and updates stack tracking information.
 * optionally stores a user data pointer that the tracker will take ownership of and delete upon reset
 * you must allocate the memory with FMemory::Malloc() or provide a custom delete function
 */
void FStackTracker::CaptureStackTrace(int32 EntriesToIgnore, void* UserData, int32 StackLen, bool bLookupStringsForAliasRemoval)
{
	// Avoid re-rentrancy as the code uses TArray/TMap.
	if( !bAvoidCapturing && bIsEnabled )
	{
		// Scoped true/ false.
		bAvoidCapturing = true;

		// Capture callstack and create CRC.
		int32 Size = (MAX_BACKTRACE_DEPTH + EntriesToIgnore) * sizeof(uint64);
		uint64* FullBackTrace = static_cast<uint64*>(FMemory_Alloca(Size));

		FMemory::Memzero(FullBackTrace, Size);

		FPlatformStackWalk::CaptureStackBackTrace( FullBackTrace, MAX_BACKTRACE_DEPTH + EntriesToIgnore );
		CA_ASSUME(FullBackTrace);

		// Skip first NUM_ENTRIES_TO_SKIP entries as they are inside this code
		uint64* BackTrace = &FullBackTrace[EntriesToIgnore];
		if (StackLen < MAX_BACKTRACE_DEPTH)
		{
			FMemory::Memzero(BackTrace + StackLen, sizeof(uint64) * (MAX_BACKTRACE_DEPTH - StackLen));
		}
		if (bLookupStringsForAliasRemoval)
		{
			for (int32 Index = 0; Index < StackLen; Index++)
			{
				if (BackTrace[Index])
				{
					uint64* Existing = AliasMap.Find(BackTrace[Index]);
					if (Existing)
					{
						BackTrace[Index] = *Existing;
					}
					else
					{
						ANSICHAR AddressInformation[512];
						AddressInformation[0] = 0;
						FPlatformStackWalk::ProgramCounterToHumanReadableString( 1, BackTrace[Index], AddressInformation, UE_ARRAY_COUNT(AddressInformation)-1 );
						FString Symbol(AddressInformation);
						int32 Spot = Symbol.Find(TEXT(" - "), ESearchCase::CaseSensitive);
						if (Spot != INDEX_NONE)
						{
							Symbol.RightChopInline(Spot + 3, EAllowShrinking::No);
						}
						Existing = StringAliasMap.Find(Symbol);
						if (Existing)
						{
							AliasMap.Add(BackTrace[Index], *Existing);
							BackTrace[Index] = *Existing;
						}
						else
						{
							AliasMap.Add(BackTrace[Index], BackTrace[Index]);
							StringAliasMap.Add(Symbol, BackTrace[Index]);
						}
					}
				}
			}
		}
		uint32 CRC = FCrc::MemCrc_DEPRECATED( BackTrace, MAX_BACKTRACE_DEPTH * sizeof(uint64) );
        
		// Use index if found
		int32* IndexPtr = CRCToCallStackIndexMap.Find( CRC );
        
		if( IndexPtr )
		{
			// Increase stack count for existing callstack.
			CallStacks[*IndexPtr].StackCount++;
			if (UpdateFn)
			{
				UpdateFn(CallStacks[*IndexPtr], UserData);
			}

			//We can delete this since the user gives ownership at the beginning of this call
			//and had a chance to update their data inside the above callback
			if (UserData)
			{
				DeleteUserDataFn(UserData);
			}
		}
		// Encountered new call stack, add to array and set index mapping.
		else
		{
			// Add to array and set mapping for future use.
			int32 Index = CallStacks.AddUninitialized();
			CRCToCallStackIndexMap.Add( CRC, Index );

			// Fill in callstack and count.
			FCallStack& CallStack = CallStacks[Index];
			FMemory::Memcpy( CallStack.Addresses, BackTrace, sizeof(uint64) * MAX_BACKTRACE_DEPTH );
			CallStack.StackCount = 1;
			CallStack.UserData = UserData;
		}

		// We're done capturing.
		bAvoidCapturing = false;
	}
	else if (UserData)
	{
		DeleteUserDataFn(UserData);
	}
}

/**
 * Dumps capture stack trace summary to the passed in log.
 */
void FStackTracker::DumpStackTraces(int32 StackThreshold, FOutputDevice& Ar, float SampleCountCorrectionFactor)
{
	check(SampleCountCorrectionFactor > 0.0f);
	// Avoid distorting results while we log them.
	check( !bAvoidCapturing );
	bAvoidCapturing = true;

	// Make a copy as sorting causes index mismatch with TMap otherwise.
	TArray<FCallStack> SortedCallStacks = CallStacks;
	// Compare function, sorting callstack by stack count in descending order.
	struct FCompareStackCount
	{
		FORCEINLINE bool operator()( const FCallStack& A, const FCallStack& B ) const { return B.StackCount < A.StackCount; }
	};
	// Sort callstacks in descending order by stack count.
	SortedCallStacks.Sort( FCompareStackCount() );

	// Iterate over each callstack to get total stack count.
	uint64 TotalStackCount = 0;
	for( int32 CallStackIndex=0; CallStackIndex<SortedCallStacks.Num(); CallStackIndex++ )
	{
		const FCallStack& CallStack = SortedCallStacks[CallStackIndex];
		TotalStackCount += CallStack.StackCount;
	}

	// Calculate the number of frames we captured.
	uint64 FramesCaptured = 0;
	if( bIsEnabled )
	{
		FramesCaptured = GFrameCounter - StartFrameCounter;
	}
	else
	{
		FramesCaptured = StopFrameCounter - StartFrameCounter;
	}
	if (!FramesCaptured)
	{
		FramesCaptured = 1;
	}

	// Log quick summary as we don't log each individual so totals in CSV won't represent real totals.
	if (SampleCountCorrectionFactor != 1.0f)
	{
		StackThreshold = FMath::Max<int32>(1, int32(float(StackThreshold) / SampleCountCorrectionFactor));
		// assume here the user has already rolled the number of frames into the correction factor
		Ar.Logf(TEXT("Captured %i unique callstacks averaging %f function calls per frame"), SortedCallStacks.Num(), float(TotalStackCount) * SampleCountCorrectionFactor);
	}
	else
	{
		Ar.Logf(TEXT("Captured %i unique callstacks totalling %i function calls over %i frames, averaging %5.2f calls/frame, Avg Per Frame"), SortedCallStacks.Num(), (int32)TotalStackCount, FramesCaptured, (float)TotalStackCount / (float)FramesCaptured);
	}

	// Iterate over each callstack and write out info in human readable form in CSV format
	for( int32 CallStackIndex=0; CallStackIndex<SortedCallStacks.Num(); CallStackIndex++ )
	{
		const FCallStack& CallStack = SortedCallStacks[CallStackIndex];

		// Avoid log spam by only logging above threshold.
		if( CallStack.StackCount > StackThreshold )
		{
			FString CallStackString;
			if (SampleCountCorrectionFactor != 1.0f)
			{
				// First row is stack count.
				CallStackString = FString::FromInt((int32)(float(CallStack.StackCount) * SampleCountCorrectionFactor));
				CallStackString += FString::Printf( TEXT(",%5.2f"), float(CallStack.StackCount) * SampleCountCorrectionFactor);
			}
			else
			{
				// First row is stack count.
				CallStackString = FString::FromInt((int32)CallStack.StackCount);
				CallStackString += FString::Printf( TEXT(",%5.2f"), static_cast<float>(CallStack.StackCount)/static_cast<float>(FramesCaptured) );
			}
			

			// Iterate over all addresses in the callstack to look up symbol name.
			for( int32 AddressIndex=0; AddressIndex<UE_ARRAY_COUNT(CallStack.Addresses) && CallStack.Addresses[AddressIndex]; AddressIndex++ )
			{
				ANSICHAR AddressInformation[512];
				AddressInformation[0] = 0;
				FPlatformStackWalk::ProgramCounterToHumanReadableString( AddressIndex, CallStack.Addresses[AddressIndex], AddressInformation, UE_ARRAY_COUNT(AddressInformation)-1 );
				CallStackString = CallStackString + TEXT(LINE_TERMINATOR_ANSI ",,,") + FString(AddressInformation);
			}

			// Finally log with ',' prefix so "Log:" can easily be discarded as row in Excel.
			Ar.Logf(TEXT(",%s"),*CallStackString);
            
			//Append any user information before moving on to the next callstack
			if (ReportFn)
			{
				ReportFn(CallStack, CallStack.StackCount, Ar);
			}
		}
		else
		{
			break;
		}
	}

	// Done logging.
	bAvoidCapturing = false;
}

/** Resets stack tracking. Deletes all user pointers passed in via CaptureStackTrace() */
void FStackTracker::ResetTracking()
{
	check(!bAvoidCapturing);
	CRCToCallStackIndexMap.Empty();

	//Clean up any user data 
	for(int32 i=0; i<CallStacks.Num(); i++)
	{
		if (CallStacks[i].UserData)
		{
			DeleteUserDataFn(CallStacks[i].UserData);
		}
	}

	CallStacks.Empty();

	// Reset the markers
	StartFrameCounter = GFrameCounter;
	StopFrameCounter = GFrameCounter;
}

/** Toggles tracking. */
void FStackTracker::ToggleTracking()
{
	ToggleTracking(!bIsEnabled, false);
}

void FStackTracker::ToggleTracking(bool bEnable, bool bSilent)
{
	if (bEnable != bIsEnabled)
	{
		bIsEnabled = bEnable;
		// Enabled
		if( bIsEnabled )
		{
			if (!bSilent)
			{
				UE_LOGF(LogStackTracker, Log, "Stack tracking is now enabled.");
			}
			StartFrameCounter = GFrameCounter;
		}
		// Disabled.
		else
		{
			StopFrameCounter = GFrameCounter;
			if (!bSilent)
			{
				UE_LOGF(LogStackTracker, Log, "Stack tracking is now disabled.");
			}
		}
	}
}
