// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_LOW_LEVEL_TESTS

#include "Containers/StaticBitArray.h"
#include "CoreMinimal.h"
#include "Misc/AutomationTest.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "TestHarness.h"

#include <catch2/generators/catch_generators.hpp>

TEST_CASE("System::Core::Containers::TStaticBitArray", "[Containers][Smoke]")
{
	SECTION("Empty bit array")
	{
		TStaticBitArray<128> Bits;
		CHECK(!Bits.HasAnyBitsSet());
		CHECK(!Bits);
		CHECK(Bits.Num() == 128);

		TStaticBitArray<128> BitsEmpty;
		CHECK(Bits == BitsEmpty);
		CHECK(!(Bits != BitsEmpty));
	}

	SECTION("One bit set")
	{
		TStaticBitArray<128> Bits;
		Bits[88] = true;
		CHECK(Bits.HasAnyBitsSet());
		CHECK(Bits);

		TStaticBitArray<128> BitsEmpty;
		CHECK(Bits != BitsEmpty);
		CHECK(!(Bits == BitsEmpty));

		TStaticBitArray<128> BitsSame;
		BitsSame[88] = true;
		CHECK(Bits == BitsSame);
		CHECK(!(Bits != BitsSame));

		TStaticBitArray<128> BitsDifferent;
		BitsDifferent[44] = true;
		CHECK(Bits != BitsDifferent);
		CHECK(!(Bits == BitsDifferent));
	}

	SECTION("Serialization")
	{
		TStaticBitArray<128> SourceBits;
		for (int32 Index = 0; Index < 128; ++Index)
		{
			SourceBits[Index] = (Index % 2 == 0);
		}

		TArray<uint8> MemoryBlob;
		FMemoryWriter WriterAr(MemoryBlob);

		WriterAr << SourceBits;

		FMemoryReader ReaderAr(MemoryBlob);

		TStaticBitArray<128> DstBits;
		ReaderAr << DstBits;

		CHECK(SourceBits == DstBits);
	}
}

#endif
