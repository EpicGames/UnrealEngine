// Copyright Epic Games, Inc. All Rights Reserved.

#include "Containers/UnrealString.h"

// Include String.cpp.inl's includes before defining the macros, in case the macros 'poison' other headers or there are re-entrant includes.
#include "Containers/StringIncludes.cpp.inl"

#define UE_STRING_CLASS             FString
#define UE_STRING_CHARTYPE          TCHAR
#define UE_STRING_CHARTYPE_IS_TCHAR 1
	#include "Containers/String.cpp.inl"
#undef UE_STRING_CHARTYPE_IS_TCHAR
#undef UE_STRING_CHARTYPE
#undef UE_STRING_CLASS

void FTextRange::CalculateLineRangesFromString(const FString& Input, TArray<FTextRange>& LineRanges)
{
	using ElementType = FString::ElementType;

	int32 LineBeginIndex = 0;

	// Loop through splitting at new-lines
	const ElementType* const InputStart = *Input;
	for (const ElementType* CurrentChar = InputStart; CurrentChar && *CurrentChar; ++CurrentChar)
	{
		// Handle a chain of \r\n slightly differently to stop the TChar<ElementType>::IsLinebreak adding two separate new-lines
		const bool bIsWindowsNewLine = (*CurrentChar == '\r' && *(CurrentChar + 1) == '\n');
		if (bIsWindowsNewLine || TChar<ElementType>::IsLinebreak(*CurrentChar))
		{
			const int32 LineEndIndex = UE_PTRDIFF_TO_INT32(CurrentChar - InputStart);
			check(LineEndIndex >= LineBeginIndex);
			LineRanges.Emplace(FTextRange(LineBeginIndex, LineEndIndex));

			if (bIsWindowsNewLine)
			{
				++CurrentChar; // skip the \n of the \r\n chain
			}
			LineBeginIndex = UE_PTRDIFF_TO_INT32(CurrentChar - InputStart) + 1; // The next line begins after the end of the current line
		}
	}

	// Process any remaining string after the last new-line
	if (LineBeginIndex <= Input.Len())
	{
		LineRanges.Emplace(FTextRange(LineBeginIndex, Input.Len()));
	}
}

namespace UE::Core::Private
{
	UE_DISABLE_OPTIMIZATION_SHIP
	void StripNegativeZero(double& InFloat)
	{
		// This works for translating a negative zero into a positive zero,
		// but if optimizations are enabled when compiling with -ffast-math
		// or /fp:fast, the compiler can strip it out.
		InFloat += 0.0f;
	}
	UE_ENABLE_OPTIMIZATION_SHIP
}
