// Copyright Epic Games, Inc. All Rights Reserved.

/*********************************************************************************************************************
 * NOTICE                                                                                                            *
 *                                                                                                                   *
 * This file is not intended to be included directly - it will be included by other .cpp files which have predefined *
 * some macros to be expanded within this file.  As such, it does not have a #pragma once as it is intended to be    *
 * included multiple times with different macro definitions.                                                         *
 *                                                                                                                   *
 * #includes needed to compile this file need to be specified in StringIncludes.cpp.inl file rather than here.       *
 *********************************************************************************************************************/
#define UE_INCLUDETOOL_IGNORE_INCONSISTENT_STATE

#if PLATFORM_MAC
#include <MacTypes.h>
#endif

#ifndef UE_STRING_CLASS
	#error "String.cpp.inl should only be included after defining UE_STRING_CLASS"
#endif
#ifndef UE_STRING_CHARTYPE
	#error "String.cpp.inl should only be included after defining UE_STRING_CHARTYPE"
#endif
#ifndef UE_STRING_CHARTYPE_IS_TCHAR
	#error "String.cpp.inl should only be included after defining UE_STRING_CHARTYPE_IS_TCHAR"
#endif

#if PLATFORM_APPLE
#import <CoreFoundation/CoreFoundation.h>
#endif

 /* String implementation
 *****************************************************************************/

namespace UE::Core::Private
{
	template <typename CompareType>
	bool MatchesWildcardRecursive(const UE_STRING_CHARTYPE* Target, int32 TargetLength, const UE_STRING_CHARTYPE* Wildcard, int32 WildcardLength, CompareType Compare)
	{
		// Skip over common initial non-wildcard-char sequence of Target and Wildcard
		for (;;)
		{
			if (WildcardLength == 0)
			{
				return TargetLength == 0;
			}

			UE_STRING_CHARTYPE WCh = *Wildcard;
			if (WCh == CHARTEXT(UE_STRING_CHARTYPE, '*') || WCh == CHARTEXT(UE_STRING_CHARTYPE, '?'))
			{
				break;
			}

			if (TargetLength == 0 || !Compare(*Target, WCh))
			{
				return false;
			}

			++Target;
			++Wildcard;
			--TargetLength;
			--WildcardLength;
		}

		// Test for common suffix. If we make it here, WildcardLength != 0, and *Wildcard
		// is an actual wildcard character. (Postcondition of previous loop.)
		const UE_STRING_CHARTYPE* TPtr = Target   + TargetLength;
		const UE_STRING_CHARTYPE* WPtr = Wildcard + WildcardLength;
		for (;;)
		{
			if (TargetLength == 0)
			{
				break;
			}

			--TPtr;
			--WPtr;

			// Since *Wildcard is a wildcard character, which we will dereference here when
			// WildcardLength is 1, we know we won't decrement WildcardLength past 1.
			UE_STRING_CHARTYPE WCh = *WPtr;
			if (WCh == CHARTEXT(UE_STRING_CHARTYPE, '*') || WCh == CHARTEXT(UE_STRING_CHARTYPE, '?'))
			{
				break;
			}

			if (!Compare(*TPtr, WCh))
			{
				return false;
			}

			--TargetLength;
			--WildcardLength;
		}

		// Match * against anything and ? against single (and zero?) chars
		UE_STRING_CHARTYPE FirstWild = *Wildcard;
		if (WildcardLength == 1 && (FirstWild == CHARTEXT(UE_STRING_CHARTYPE, '*') || TargetLength < 2))
		{
			return true;
		}
		++Wildcard;
		--WildcardLength;

		// This routine is very slow, though it does ok with one wildcard
		int32 MaxNum = TargetLength;
		if (FirstWild == CHARTEXT(UE_STRING_CHARTYPE, '?') && MaxNum > 1)
		{
			MaxNum = 1;
		}

		for (int32 Index = 0; Index <= MaxNum; ++Index)
		{
			if (MatchesWildcardRecursive(Target + Index, TargetLength - Index, Wildcard, WildcardLength, Compare))
			{
				return true;
			}
		}
		return false;
	}

	template <typename SrcCharType>
	void AppendCharacters(TArray<UE_STRING_CHARTYPE>& Out, const SrcCharType* Str, int32 Count)
	{
		check(Count >= 0);

		if (!Count)
		{
			return;
		}

		checkSlow(Str);

		int32 OldEnd = Out.Num();
	
		// Try to reserve enough space by guessing that the new length will be the same as the input length.
		// Include an extra gap for a null terminator if we don't already have a string allocated
		Out.AddUninitialized(Count + (OldEnd ? 0 : 1));
		OldEnd -= OldEnd ? 1 : 0;

		UE_STRING_CHARTYPE* Dest = Out.GetData() + OldEnd;

		// Try copying characters to end of string, overwriting null terminator if we already have one
		UE_STRING_CHARTYPE* NewEnd = FPlatformString::Convert(Dest, Count, Str, Count);
		if (!NewEnd)
		{
			// If that failed, it will have meant that conversion likely contained multi-code unit characters
			// and so the buffer wasn't long enough, so calculate it properly.
			int32 Length = FPlatformString::ConvertedLength<UE_STRING_CHARTYPE>(Str, Count);

			// Add the extra bytes that we need
			Out.AddUninitialized(Length - Count);

			// Restablish destination pointer in case a realloc happened
			Dest = Out.GetData() + OldEnd;

			NewEnd = FPlatformString::Convert(Dest, Length, Str, Count);
			checkSlow(NewEnd);
		}
		else
		{
			int32 NewEndIndex = (int32)(NewEnd - Dest);
			if (NewEndIndex < Count)
			{
				Out.SetNumUninitialized(OldEnd + NewEndIndex + 1, EAllowShrinking::No);
			}
		}

		// (Re-)establish the null terminator
		*NewEnd = CHARTEXT(UE_STRING_CHARTYPE, '\0');
	}

	template <typename SrcCharType>
	FORCEINLINE void ConstructFromCString(/* Out */ TArray<UE_STRING_CHARTYPE>& Data, const SrcCharType* Src, int32 ExtraSlack)
	{
		checkf(Src,             TEXT("Unable to construct string from a null pointer"));
		checkf(ExtraSlack >= 0, TEXT("Unable to construct string with negative slack"));

		if (*Src)
		{
			int32 SrcLen  = TCString<SrcCharType>::Strlen(Src) + 1;
			int32 DestLen = FPlatformString::ConvertedLength<UE_STRING_CHARTYPE>(Src, SrcLen);
			Data.Reserve(DestLen + ExtraSlack);
			Data.AddUninitialized(DestLen);

			FPlatformString::Convert(Data.GetData(), DestLen, Src, SrcLen);
		}
		else if (ExtraSlack > 0)
		{
			Data.Reserve(ExtraSlack + 1);
		}
	}

	template <typename SrcCharType>
	FORCEINLINE void ConstructWithLength(/* Out */ TArray<UE_STRING_CHARTYPE>& Data, const SrcCharType* InSrc, int32 InCount, int32 ExtraSlack)
	{
		checkf(InSrc || InCount == 0, TEXT("Unable to construct string from a null pointer"));
		checkf(InCount >= 0,          TEXT("Unable to construct string with a negative size"));
		checkf(ExtraSlack >= 0,       TEXT("Unable to construct string with negative slack"));

		int32 DestLen = FPlatformString::ConvertedLength<UE_STRING_CHARTYPE>(InSrc, InCount);
		if (DestLen > 0)
		{
			Data.Reserve(DestLen + ExtraSlack + 1);
			Data.AddUninitialized(DestLen + 1);

			UE_STRING_CHARTYPE* DataPtr = Data.GetData();

			FPlatformString::Convert(DataPtr, DestLen, InSrc, InCount);
			DataPtr[DestLen] = CHARTEXT(UE_STRING_CHARTYPE, '\0');
		}
		else if (ExtraSlack > 0)
		{
			Data.Reserve(ExtraSlack + 1);
		}
	}

	template <typename SrcCharType>
	FORCEINLINE void ConstructWithSlack(/* Out */ TArray<UE_STRING_CHARTYPE>& Data, const SrcCharType* Src, int32 ExtraSlack)
	{
		checkf(Src,             TEXT("Unable to construct string from a null pointer"));
		checkf(ExtraSlack >= 0, TEXT("Unable to construct string with negative slack"));

		if (*Src)
		{
			int32 SrcLen = TCString<SrcCharType>::Strlen(Src) + 1;
			int32 DestLen = FPlatformString::ConvertedLength<UE_STRING_CHARTYPE>(Src, SrcLen);
			Data.Reserve(DestLen + ExtraSlack);
			Data.AddUninitialized(DestLen);

			FPlatformString::Convert(Data.GetData(), DestLen, Src, SrcLen);
		}
		else if (ExtraSlack > 0)
		{
			Data.Reserve(ExtraSlack + 1); 
		}
	}
} // namespace UE::Core::Private

UE_STRING_CLASS::UE_STRING_CLASS(const ANSICHAR* Str)								{ if (Str) { UE::Core::Private::ConstructFromCString(Data, Str, 0); } }
UE_STRING_CLASS::UE_STRING_CLASS(const WIDECHAR* Str)								{ if (Str) { UE::Core::Private::ConstructFromCString(Data, Str, 0); } }
UE_STRING_CLASS::UE_STRING_CLASS(const UTF8CHAR* Str)								{ if (Str) { UE::Core::Private::ConstructFromCString(Data, Str, 0); } }
UE_STRING_CLASS::UE_STRING_CLASS(const UCS2CHAR* Str)								{ if (Str) { UE::Core::Private::ConstructFromCString(Data, Str, 0); } }
UE_STRING_CLASS::UE_STRING_CLASS(const UTF32CHAR* Str)								{ if (Str) { UE::Core::Private::ConstructFromCString(Data, Str, 0); } }

// Deprecated
UE_STRING_CLASS::UE_STRING_CLASS(int32 Len, const ANSICHAR* Str)					{ if (Str && *Str) { UE::Core::Private::ConstructWithLength(Data, Str, Len, 0); } }
UE_STRING_CLASS::UE_STRING_CLASS(int32 Len, const WIDECHAR* Str)					{ if (Str && *Str) { UE::Core::Private::ConstructWithLength(Data, Str, Len, 0); } }
UE_STRING_CLASS::UE_STRING_CLASS(int32 Len, const UTF8CHAR* Str)					{ if (Str && *Str) { UE::Core::Private::ConstructWithLength(Data, Str, Len, 0); } }
UE_STRING_CLASS::UE_STRING_CLASS(int32 Len, const UCS2CHAR* Str)					{ if (Str && *Str) { UE::Core::Private::ConstructWithLength(Data, Str, Len, 0); } }
UE_STRING_CLASS::UE_STRING_CLASS(const ANSICHAR* Str, int32 ExtraSlack)				{ if (Str) { UE::Core::Private::ConstructFromCString(Data, Str, ExtraSlack); } else if (ExtraSlack > 0) { Data.Reserve(ExtraSlack + 1); } }
UE_STRING_CLASS::UE_STRING_CLASS(const WIDECHAR* Str, int32 ExtraSlack)				{ if (Str) { UE::Core::Private::ConstructFromCString(Data, Str, ExtraSlack); } else if (ExtraSlack > 0) { Data.Reserve(ExtraSlack + 1); } }
UE_STRING_CLASS::UE_STRING_CLASS(const UTF8CHAR* Str, int32 ExtraSlack)				{ if (Str) { UE::Core::Private::ConstructFromCString(Data, Str, ExtraSlack); } else if (ExtraSlack > 0) { Data.Reserve(ExtraSlack + 1); } }
UE_STRING_CLASS::UE_STRING_CLASS(const UCS2CHAR* Str, int32 ExtraSlack)				{ if (Str) { UE::Core::Private::ConstructFromCString(Data, Str, ExtraSlack); } else if (ExtraSlack > 0) { Data.Reserve(ExtraSlack + 1); } }
// Deprecated

UE_STRING_CLASS UE_STRING_CLASS::ConstructFromPtrSize(const ANSICHAR* Str, int32 Len)								{ UE_STRING_CLASS Result; UE::Core::Private::ConstructWithLength(Result.Data, Str, Len, 0); return Result; }
UE_STRING_CLASS UE_STRING_CLASS::ConstructFromPtrSize(const WIDECHAR* Str, int32 Len)								{ UE_STRING_CLASS Result; UE::Core::Private::ConstructWithLength(Result.Data, Str, Len, 0); return Result; }
UE_STRING_CLASS UE_STRING_CLASS::ConstructFromPtrSize(const UTF8CHAR* Str, int32 Len)								{ UE_STRING_CLASS Result; UE::Core::Private::ConstructWithLength(Result.Data, Str, Len, 0); return Result; }
UE_STRING_CLASS UE_STRING_CLASS::ConstructFromPtrSize(const UCS2CHAR* Str, int32 Len)								{ UE_STRING_CLASS Result; UE::Core::Private::ConstructWithLength(Result.Data, Str, Len, 0); return Result; }
UE_STRING_CLASS UE_STRING_CLASS::ConstructFromPtrSize(const UTF32CHAR* Str, int32 Len)								{ UE_STRING_CLASS Result; UE::Core::Private::ConstructWithLength(Result.Data, Str, Len, 0); return Result; }
UE_STRING_CLASS UE_STRING_CLASS::ConstructWithSlack(const ANSICHAR* Str, int32 ExtraSlack)							{ UE_STRING_CLASS Result; UE::Core::Private::ConstructFromCString(Result.Data, Str, ExtraSlack); return Result; }
UE_STRING_CLASS UE_STRING_CLASS::ConstructWithSlack(const WIDECHAR* Str, int32 ExtraSlack)							{ UE_STRING_CLASS Result; UE::Core::Private::ConstructFromCString(Result.Data, Str, ExtraSlack); return Result; }
UE_STRING_CLASS UE_STRING_CLASS::ConstructWithSlack(const UTF8CHAR* Str, int32 ExtraSlack)							{ UE_STRING_CLASS Result; UE::Core::Private::ConstructFromCString(Result.Data, Str, ExtraSlack); return Result; }
UE_STRING_CLASS UE_STRING_CLASS::ConstructWithSlack(const UCS2CHAR* Str, int32 ExtraSlack)							{ UE_STRING_CLASS Result; UE::Core::Private::ConstructFromCString(Result.Data, Str, ExtraSlack); return Result; }
UE_STRING_CLASS UE_STRING_CLASS::ConstructWithSlack(const UTF32CHAR* Str, int32 ExtraSlack)							{ UE_STRING_CLASS Result; UE::Core::Private::ConstructFromCString(Result.Data, Str, ExtraSlack); return Result; }
UE_STRING_CLASS UE_STRING_CLASS::ConstructFromPtrSizeWithSlack(const ANSICHAR* Str, int32 Len, int32 ExtraSlack)	{ UE_STRING_CLASS Result; UE::Core::Private::ConstructWithLength(Result.Data, Str, Len, ExtraSlack); return Result; }
UE_STRING_CLASS UE_STRING_CLASS::ConstructFromPtrSizeWithSlack(const WIDECHAR* Str, int32 Len, int32 ExtraSlack)	{ UE_STRING_CLASS Result; UE::Core::Private::ConstructWithLength(Result.Data, Str, Len, ExtraSlack); return Result; }
UE_STRING_CLASS UE_STRING_CLASS::ConstructFromPtrSizeWithSlack(const UTF8CHAR* Str, int32 Len, int32 ExtraSlack)	{ UE_STRING_CLASS Result; UE::Core::Private::ConstructWithLength(Result.Data, Str, Len, ExtraSlack); return Result; }
UE_STRING_CLASS UE_STRING_CLASS::ConstructFromPtrSizeWithSlack(const UCS2CHAR* Str, int32 Len, int32 ExtraSlack)	{ UE_STRING_CLASS Result; UE::Core::Private::ConstructWithLength(Result.Data, Str, Len, ExtraSlack); return Result; }
UE_STRING_CLASS UE_STRING_CLASS::ConstructFromPtrSizeWithSlack(const UTF32CHAR* Str, int32 Len, int32 ExtraSlack)	{ UE_STRING_CLASS Result; UE::Core::Private::ConstructWithLength(Result.Data, Str, Len, ExtraSlack); return Result; }

UE_STRING_CLASS& UE_STRING_CLASS::operator=(const ElementType* Other)
{
	if (Data.GetData() != Other)
	{
		int32 Len = (Other && *Other) ? TCString<ElementType>::Strlen(Other) + 1 : 0;
		Data.Empty(Len);
		Data.AddUninitialized(Len);

		if (Len)
		{
			FMemory::Memcpy(Data.GetData(), Other, Len * sizeof(ElementType));
		}
	}
	return *this;
}


void UE_STRING_CLASS::AssignRange(const ElementType* OtherData, int32 OtherLen)
{
	if (OtherLen == 0)
	{
		Empty();
	}
	else
	{
		const int32 ThisLen = Len();
		if (OtherLen <= ThisLen)
		{
			// Unless the input is longer, this might be assigned from a view of itself.
			ElementType* DataPtr = Data.GetData();
			FMemory::Memmove(DataPtr, OtherData, OtherLen * sizeof(ElementType));
			DataPtr[OtherLen] = CHARTEXT(ElementType, '\0');
			Data.RemoveAt(OtherLen + 1, ThisLen - OtherLen);
		}
		else
		{
			Data.Empty(OtherLen + 1);
			Data.AddUninitialized(OtherLen + 1);
			ElementType* DataPtr = Data.GetData();
			FMemory::Memcpy(DataPtr, OtherData, OtherLen * sizeof(ElementType));
			DataPtr[OtherLen] = CHARTEXT(ElementType, '\0');
		}
	}
}

void UE_STRING_CLASS::Reserve(int32 CharacterCount)
{
	checkSlow(CharacterCount >= 0 && CharacterCount < MAX_int32);
	if (CharacterCount > 0)
	{
		Data.Reserve(CharacterCount + 1);
	}	
}

void UE_STRING_CLASS::Empty(int32 Slack)
{
	Data.Empty(Slack ? Slack + 1 : 0);
}

void UE_STRING_CLASS::Empty()
{
	Data.Empty(0);
}

void UE_STRING_CLASS::Reset(int32 NewReservedSize)
{
	const int32 NewSizeIncludingTerminator = (NewReservedSize > 0) ? (NewReservedSize + 1) : 0;
	Data.Reset(NewSizeIncludingTerminator);
	if (ElementType* DataPtr = Data.GetData())
	{
		*DataPtr = CHARTEXT(ElementType, '\0');
	}
}

void UE_STRING_CLASS::Shrink()
{
	Data.Shrink();
}

#ifdef __OBJC__
/** Convert string to Objective-C NSString */
NSString* UE_STRING_CLASS::GetNSString() const
{
    NSString* OutString = (NSString*)GetCFString();
    [OutString autorelease];
    
    return OutString;
}
#endif

#if PLATFORM_APPLE && UE_STRING_CHARTYPE_IS_TCHAR

#include "Apple/AppleStringUtils.h"
#include "Misc/AssertionMacros.h"
UE_STRING_CLASS::UE_STRING_CLASS(const CFStringRef In)
{
	uint32_t StringLength = In ? CFStringGetLength(In) : 0;

	if (StringLength > 0)
	{
		// Convert the NSString data into the native TCHAR format for UE4
		// This returns a buffer of bytes, but they can be safely cast to a buffer of TCHARs
		const CFStringEncoding Encoding = kCFStringEncodingUTF16LE;
		CFRange Range = CFRangeMake(0, StringLength);
		CFIndex BytesNeeded;
		if (CFStringGetBytes(In, Range, Encoding, '?', false, NULL, 0, &BytesNeeded) > 0)
		{
			const size_t Length = BytesNeeded / sizeof(TCHAR);
			Data.Reserve(Length + 1);
			Data.AddUninitialized(Length + 1);
			CFStringGetBytes(In, Range, Encoding, '?', false, (uint8*)Data.GetData(), Length * sizeof(TCHAR) + 1, NULL);
			Data[Length] = 0;
		}
	}
}

#endif

#if PLATFORM_APPLE
/** Convert string to bridgeable CFString */
CFStringRef UE_STRING_CLASS::GetCFString() const
{
	return CFStringCreateWithBytes(kCFAllocatorDefault, (UInt8*)Data.GetData(), Len() * sizeof(ElementType), kCFStringEncodingUTF16LE, false);
}
#endif

UE_STRING_CLASS& UE_STRING_CLASS::AppendChar(ElementType InChar)
{
	CheckInvariants();

	if (InChar != 0)
	{
		// position to insert the character.  
		// At the end of the string if we have existing characters, otherwise at the 0 position
		int32 InsertIndex = (Data.Num() > 0) ? Data.Num()-1 : 0;	

		// number of characters to add.  If we don't have any existing characters, 
		// we'll need to append the terminating zero as well.
		int32 InsertCount = (Data.Num() > 0) ? 1 : 2;

		Data.AddUninitialized(InsertCount);
		Data[InsertIndex] = InChar;
		Data[InsertIndex+1] = CHARTEXT(ElementType, '\0');
	}
	return *this;
}

void UE_STRING_CLASS::AppendChars(const ANSICHAR* Str, int32 Count)
{
	CheckInvariants();
	UE::Core::Private::AppendCharacters(Data, Str, Count);
}

void UE_STRING_CLASS::AppendChars(const WIDECHAR* Str, int32 Count)
{
	CheckInvariants();
	UE::Core::Private::AppendCharacters(Data, Str, Count);
}

void UE_STRING_CLASS::AppendChars(const UCS2CHAR* Str, int32 Count)
{
	CheckInvariants();
	UE::Core::Private::AppendCharacters(Data, Str, Count);
}

void UE_STRING_CLASS::AppendChars(const UTF8CHAR* Str, int32 Count)
{
	CheckInvariants();
	UE::Core::Private::AppendCharacters(Data, Str, Count);
}

void UE_STRING_CLASS::AppendChars(const UTF32CHAR* Str, int32 Count)
{
	CheckInvariants();
	UE::Core::Private::AppendCharacters(Data, Str, Count);
}

void UE_STRING_CLASS::TrimToNullTerminator()
{
	if (Data.Num())
	{
		int32 DataLen = TCString<ElementType>::Strlen(Data.GetData());
		check(DataLen == 0 || DataLen < Data.Num());
		int32 Len = DataLen > 0 ? DataLen+1 : 0;

		check(Len <= Data.Num());
		Data.RemoveAt(Len, Data.Num()-Len);
	}
}


int32 UE_STRING_CLASS::Find(const ElementType* SubStr, int32 SubStrLen, ESearchCase::Type SearchCase, ESearchDir::Type SearchDir, int32 StartPosition) const
{
	checkf(SubStrLen >= 0, TEXT("Invalid SubStrLen: %d"), SubStrLen);

	if (SearchDir == ESearchDir::FromStart)
	{
		const ElementType* Start = **this;
		int32 RemainingLength = Len();
		if (StartPosition != INDEX_NONE && RemainingLength > 0)
		{
			const ElementType* End = Start + RemainingLength;
			Start += FMath::Clamp(StartPosition, 0, RemainingLength - 1);
			RemainingLength = UE_PTRDIFF_TO_INT32(End - Start);
		}
		const ElementType* Tmp = SearchCase == ESearchCase::IgnoreCase
			? TCString<ElementType>::Strnistr(Start, RemainingLength, SubStr, SubStrLen)
			: TCString<ElementType>::Strnstr(Start, RemainingLength, SubStr, SubStrLen);

		return Tmp ? UE_PTRDIFF_TO_INT32(Tmp-**this) : INDEX_NONE;
	}
	else
	{
		// if ignoring, do a onetime ToUpper on both strings, to avoid ToUppering multiple
		// times in the loop below
		if (SearchCase == ESearchCase::IgnoreCase)
		{
			return ToUpper().Find(UE_STRING_CLASS::ConstructFromPtrSize(SubStr, SubStrLen).ToUpper(), ESearchCase::CaseSensitive, SearchDir, StartPosition);
		}
		else
		{
			const int32 SearchStringLength=FMath::Max(1, SubStrLen);
			
			if (StartPosition == INDEX_NONE || StartPosition >= Len())
			{
				StartPosition = Len();
			}
			
			for (int32 i = StartPosition - SearchStringLength; i >= 0; i--)
			{
				int32 j;
				for (j=0; j != SubStrLen; j++)
				{
					if ((*this)[i+j]!=SubStr[j])
					{
						break;
					}
				}
				
				if (j == SubStrLen)
				{
					return i;
				}
			}
			return INDEX_NONE;
		}
	}
}

bool UE_STRING_CLASS::Split(const UE_STRING_CLASS& InStr, UE_STRING_CLASS* LeftStr, UE_STRING_CLASS* RightStr, ESearchCase::Type SearchCase, ESearchDir::Type SearchDir) const
{
	check(LeftStr != RightStr || LeftStr == nullptr);

	int32 InPos = Find(InStr, SearchCase, SearchDir);

	if (InPos < 0)
	{
		return false;
	}

	if (LeftStr)
	{
		if (LeftStr != this)
		{
			*LeftStr = Left(InPos);
			if (RightStr)
			{
				*RightStr = Mid(InPos + InStr.Len());
			}
		}
		else
		{
			// we know that RightStr can't be this so we can safely modify it before we deal with LeftStr
			if (RightStr)
			{
				*RightStr = Mid(InPos + InStr.Len());
			}
			*LeftStr = Left(InPos);
		}
	}
	else if (RightStr)
	{
		*RightStr = Mid(InPos + InStr.Len());
	}

	return true;
}

bool UE_STRING_CLASS::Split(const UE_STRING_CLASS& InStr, UE_STRING_CLASS* LeftStr, UE_STRING_CLASS* RightStr) const
{
	return Split(InStr, LeftStr, RightStr, ESearchCase::IgnoreCase);
}

UE_STRING_CLASS UE_STRING_CLASS::ToUpper() const &
{
	UE_STRING_CLASS New = *this;
	New.ToUpperInline();
	return New;
}

UE_STRING_CLASS UE_STRING_CLASS::ToUpper() &&
{
	this->ToUpperInline();
	return MoveTemp(*this);
}

void UE_STRING_CLASS::ToUpperInline()
{
	const int32 StringLength = Len();
	ElementType* RawData = Data.GetData();
	for (int32 i = 0; i < StringLength; ++i)
	{
		RawData[i] = TChar<ElementType>::ToUpper(RawData[i]);
	}
}


UE_STRING_CLASS UE_STRING_CLASS::ToLower() const &
{
	UE_STRING_CLASS New = *this;
	New.ToLowerInline();
	return New;
}

UE_STRING_CLASS UE_STRING_CLASS::ToLower() &&
{
	this->ToLowerInline();
	return MoveTemp(*this);
}

void UE_STRING_CLASS::ToLowerInline()
{
	const int32 StringLength = Len();
	ElementType* RawData = Data.GetData();
	for (int32 i = 0; i < StringLength; ++i)
	{
		RawData[i] = TChar<ElementType>::ToLower(RawData[i]);
	}
}

void UE_STRING_CLASS::RemoveSpacesInline()
{
	const int32 StringLength = Len();
	if (StringLength == 0)
	{
		return;
	}

	ElementType* RawData = Data.GetData();
	int32 CopyToIndex = 0;
	for (int32 CopyFromIndex = 0; CopyFromIndex < StringLength; ++CopyFromIndex)
	{
		if (RawData[CopyFromIndex] != ' ')
		{	// Copy any character OTHER than space.
			RawData[CopyToIndex] = RawData[CopyFromIndex];
			++CopyToIndex;
		}
	}

	// Copy null-terminating character.
	if (CopyToIndex <= StringLength)
	{
		RawData[CopyToIndex] = CHARTEXT(ElementType, '\0');
		Data.SetNum(CopyToIndex + 1, EAllowShrinking::No);
	}
}

bool UE_STRING_CLASS::StartsWith(const ElementType* InPrefix, int32 InPrefixLen, ESearchCase::Type SearchCase) const
{
	if (SearchCase == ESearchCase::IgnoreCase)
	{
		return InPrefixLen > 0 && !TCString<ElementType>::Strnicmp(**this, InPrefix, InPrefixLen);
	}
	else
	{
		return InPrefixLen > 0 && !TCString<ElementType>::Strncmp(**this, InPrefix, InPrefixLen);
	}
}

bool UE_STRING_CLASS::EndsWith(const ElementType* InSuffix, int32 InSuffixLen, ESearchCase::Type SearchCase) const
{
	if (SearchCase == ESearchCase::IgnoreCase)
	{
		return InSuffixLen > 0 &&
			Len() >= InSuffixLen &&
			!TCString<ElementType>::Stricmp(&(*this)[ Len() - InSuffixLen ], InSuffix);
	}
	else
	{
		return InSuffixLen > 0 &&
			Len() >= InSuffixLen &&
			!TCString<ElementType>::Strcmp(&(*this)[ Len() - InSuffixLen ], InSuffix);
	}
}

void UE_STRING_CLASS::InsertAt(int32 Index, ElementType Character)
{
	if (Character != 0)
	{
		if (Data.Num() == 0)
		{
			*this += Character;
		}
		else
		{
			Data.Insert(Character, Index);
		}
	}
}

void UE_STRING_CLASS::InsertAt(int32 Index, const UE_STRING_CLASS& Characters)
{
	if (Characters.Len())
	{
		if (Data.Num() == 0)
		{
			*this += Characters;
		}
		else
		{
			Data.Insert(Characters.Data.GetData(), Characters.Len(), Index);
		}
	}
}

void UE_STRING_CLASS::RemoveAt(int32 Index, EAllowShrinking AllowShrinking)
{
	// The clamping behavior below that we are inheriting from the (Index, Count, AllowShrinking) overload is unfortunate, as it pessimizes this call.
	if (FMath::Clamp(1, 0, Len()-Index) == 1)
	{
		Data.RemoveAt(Index, AllowShrinking);
	}
}

void UE_STRING_CLASS::RemoveAt(int32 Index, int32 Count, EAllowShrinking AllowShrinking)
{
	Data.RemoveAt(Index, FMath::Clamp(Count, 0, Len()-Index), AllowShrinking);
}

bool UE_STRING_CLASS::RemoveFromStart(const ElementType* InPrefix, int32 InPrefixLen, ESearchCase::Type SearchCase)
{
	if (InPrefixLen == 0)
	{
		return false;
	}

	if (StartsWith(InPrefix, InPrefixLen, SearchCase))
	{
		RemoveAt(0, InPrefixLen);
		return true;
	}

	return false;
}

bool UE_STRING_CLASS::RemoveFromEnd(const ElementType* InSuffix, int32 InSuffixLen, ESearchCase::Type SearchCase)
{
	if (InSuffixLen == 0)
	{
		return false;
	}

	if (EndsWith(InSuffix, InSuffixLen, SearchCase))
	{
		RemoveAt(Len() - InSuffixLen, InSuffixLen);
		return true;
	}

	return false;
}

namespace UE::String::Private
{

template <typename LhsType, typename RhsType>
[[nodiscard]] FORCEINLINE UE_STRING_CLASS UE_JOIN(ConcatStrings_, UE_STRING_CLASS)(LhsType&& Lhs, RhsType&& Rhs)
{
	Lhs.CheckInvariants();
	Rhs.CheckInvariants();

	if (Lhs.IsEmpty())
	{
		return Forward<RhsType>(Rhs);
	}

	int32 RhsLen = Rhs.Len();

	UE_STRING_CLASS Result(Forward<LhsType>(Lhs), /* extra slack */ RhsLen);
	Result.AppendChars(Rhs.GetCharArray().GetData(), RhsLen);
		
	return Result;
}

template <typename LhsCharType, typename RhsType>
[[nodiscard]] FORCEINLINE UE_STRING_CLASS UE_JOIN(ConcatRangeString_, UE_STRING_CLASS)(const LhsCharType* Lhs, int32 LhsLen, RhsType&& Rhs)
{
	using ElementType = UE_STRING_CLASS::ElementType;

	checkSlow(LhsLen >= 0);
	Rhs.CheckInvariants();
	if (LhsLen == 0)
	{
		return Forward<RhsType>(Rhs);
	}

	int32 RhsLen = Rhs.Len();

	// This is not entirely optimal, as if the Rhs is an rvalue and has enough slack space to hold Lhs, then
	// the memory could be reused here without constructing a new object.  However, until there is proof otherwise,
	// I believe this will be relatively rare and isn't worth making the code a lot more complex right now.
	UE_STRING_CLASS Result;
	Result.GetCharArray().Reserve(LhsLen + RhsLen + 1);
	Result.GetCharArray().AddUninitialized(LhsLen + RhsLen + 1);

	ElementType* ResultData = Result.GetCharArray().GetData();
	CopyAssignItems(ResultData, Lhs, LhsLen);
	CopyAssignItems(ResultData + LhsLen, Rhs.GetCharArray().GetData(), RhsLen);
	*(ResultData + LhsLen + RhsLen) = CHARTEXT(ElementType, '\0');

	return Result;
}

template <typename LhsType, typename RhsCharType>
[[nodiscard]] FORCEINLINE UE_STRING_CLASS UE_JOIN(ConcatStringRange_, UE_STRING_CLASS)(LhsType&& Lhs, const RhsCharType* Rhs, int32 RhsLen)
{
	Lhs.CheckInvariants();
	checkSlow(RhsLen >= 0);
	if (RhsLen == 0)
	{
		return Forward<LhsType>(Lhs);
	}

	UE_STRING_CLASS Result(Forward<LhsType>(Lhs), /* extra slack */ RhsLen);
	Result.AppendChars(Rhs, RhsLen);
		
	return Result;
}

template <typename LhsCharType, typename RhsType>
[[nodiscard]] FORCEINLINE UE_STRING_CLASS UE_JOIN(ConcatPtrString_, UE_STRING_CLASS)(const LhsCharType* Lhs, RhsType&& Rhs)
{
	checkSlow(Lhs);
	if (!Lhs)
	{
		return Forward<RhsType>(Rhs);
	}

	return UE_JOIN(ConcatRangeString_, UE_STRING_CLASS)(Lhs, TCString<LhsCharType>::Strlen(Lhs), Forward<RhsType>(Rhs));
}

template <typename LhsType, typename RhsCharType>
[[nodiscard]] FORCEINLINE UE_STRING_CLASS UE_JOIN(ConcatStringPtr_, UE_STRING_CLASS)(LhsType&& Lhs, const RhsCharType* Rhs)
{
	checkSlow(Rhs);
	if (!Rhs)
	{
		return Forward<LhsType>(Lhs);
	}
	return UE_JOIN(ConcatStringRange_, UE_STRING_CLASS)(Forward<LhsType>(Lhs), Rhs, TCString<RhsCharType>::Strlen(Rhs));
}

} // namespace UE::String::Private

UE_STRING_CLASS UE_STRING_CLASS::ConcatFF(const UE_STRING_CLASS& Lhs, const UE_STRING_CLASS& Rhs)				{ return UE::String::Private::UE_JOIN(ConcatStrings_,     UE_STRING_CLASS)(Lhs, Rhs); }
UE_STRING_CLASS UE_STRING_CLASS::ConcatFF(UE_STRING_CLASS&& Lhs, const UE_STRING_CLASS& Rhs)					{ return UE::String::Private::UE_JOIN(ConcatStrings_,     UE_STRING_CLASS)(MoveTemp(Lhs), Rhs); }
UE_STRING_CLASS UE_STRING_CLASS::ConcatFF(const UE_STRING_CLASS& Lhs, UE_STRING_CLASS&& Rhs)					{ return UE::String::Private::UE_JOIN(ConcatStrings_,     UE_STRING_CLASS)(Lhs, MoveTemp(Rhs)); }
UE_STRING_CLASS UE_STRING_CLASS::ConcatFF(UE_STRING_CLASS&& Lhs, UE_STRING_CLASS&& Rhs)							{ return UE::String::Private::UE_JOIN(ConcatStrings_,     UE_STRING_CLASS)(MoveTemp(Lhs), MoveTemp(Rhs)); }
UE_STRING_CLASS UE_STRING_CLASS::ConcatFC(const UE_STRING_CLASS& Lhs, const ElementType* Rhs)					{ return UE::String::Private::UE_JOIN(ConcatStringPtr_,   UE_STRING_CLASS)(Lhs, Rhs); }
UE_STRING_CLASS UE_STRING_CLASS::ConcatFC(UE_STRING_CLASS&& Lhs, const ElementType* Rhs)						{ return UE::String::Private::UE_JOIN(ConcatStringPtr_,   UE_STRING_CLASS)(MoveTemp(Lhs), Rhs); }
UE_STRING_CLASS UE_STRING_CLASS::ConcatCF(const ElementType* Lhs,	const UE_STRING_CLASS& Rhs)					{ return UE::String::Private::UE_JOIN(ConcatPtrString_,   UE_STRING_CLASS)(Lhs, Rhs); }
UE_STRING_CLASS UE_STRING_CLASS::ConcatCF(const ElementType* Lhs, UE_STRING_CLASS&& Rhs)						{ return UE::String::Private::UE_JOIN(ConcatPtrString_,   UE_STRING_CLASS)(Lhs, MoveTemp(Rhs)); }
UE_STRING_CLASS UE_STRING_CLASS::ConcatFR(const UE_STRING_CLASS& Lhs, const ElementType* Rhs, int32 RhsLen)		{ return UE::String::Private::UE_JOIN(ConcatStringRange_, UE_STRING_CLASS)(Lhs, Rhs, RhsLen); }
UE_STRING_CLASS UE_STRING_CLASS::ConcatFR(UE_STRING_CLASS&& Lhs, const ElementType* Rhs, int32 RhsLen)			{ return UE::String::Private::UE_JOIN(ConcatStringRange_, UE_STRING_CLASS)(MoveTemp(Lhs), Rhs, RhsLen); }
UE_STRING_CLASS UE_STRING_CLASS::ConcatRF(const ElementType* Lhs, int32 LhsLen, const UE_STRING_CLASS& Rhs)		{ return UE::String::Private::UE_JOIN(ConcatRangeString_, UE_STRING_CLASS)(Lhs, LhsLen, Rhs); }
UE_STRING_CLASS UE_STRING_CLASS::ConcatRF(const ElementType* Lhs, int32 LhsLen, UE_STRING_CLASS&& Rhs)			{ return UE::String::Private::UE_JOIN(ConcatRangeString_, UE_STRING_CLASS)(Lhs, LhsLen, MoveTemp(Rhs)); }

/**
 * Concatenate this path with given path ensuring the / character is used between them
 *
 * @param Str       Pointer to an array of TCHARs (not necessarily null-terminated) to be concatenated onto the end of this.
 * @param StrLength Exact number of characters from Str to append.
 */
void UE_STRING_CLASS::PathAppend(const ElementType* Str, int32 StrLength)
{
	int32 DataNum = Data.Num();
	if (StrLength == 0)
	{
		if (DataNum > 1 && Data[DataNum - 2] != CHARTEXT(ElementType, '/') && Data[DataNum - 2] != CHARTEXT(ElementType, '\\'))
		{
			Data[DataNum - 1] = CHARTEXT(ElementType, '/');
			Data.Add(CHARTEXT(ElementType, '\0'));
		}
	}
	else
	{
		if (DataNum > 0)
		{
			if (DataNum > 1 && Data[DataNum - 2] != CHARTEXT(ElementType, '/') && Data[DataNum - 2] != CHARTEXT(ElementType, '\\') && *Str != CHARTEXT(ElementType, '/'))
			{
				Data[DataNum - 1] = CHARTEXT(ElementType, '/');
			}
			else
			{
				Data.Pop(EAllowShrinking::No);
				--DataNum;
			}
		}

		Reserve(DataNum + StrLength);
		Data.Append(Str, StrLength);
		Data.Add(CHARTEXT(ElementType, '\0'));
	}
}

UE_STRING_CLASS UE_STRING_CLASS::RightChop(int32 Count) const &
{
	const int32 Length = Len();
	const int32 Skip = FMath::Clamp(Count, 0, Length);
	return UE_STRING_CLASS::ConstructFromPtrSize(**this + Skip, Length - Skip);
}

UE_STRING_CLASS UE_STRING_CLASS::Mid(int32 Start, int32 Count) const &
{
	if (Count >= 0)
	{
		const int32 Length = Len();
		const int32 RequestedStart = Start;
		Start = FMath::Clamp(Start, 0, Length);
		const int32 End = (int32)FMath::Clamp((int64)Count + RequestedStart, (int64)Start, (int64)Length);
		return UE_STRING_CLASS::ConstructFromPtrSize(**this + Start, End-Start);
	}

	return UE_STRING_CLASS();
}

UE_STRING_CLASS UE_STRING_CLASS::Mid(int32 Start, int32 Count) &&
{
	MidInline(Start, Count, EAllowShrinking::No);
	return MoveTemp(*this);
}

void UE_STRING_CLASS::ReplaceCharInlineCaseSensitive(const ElementType From, const ElementType To)
{
	for (ElementType& Character : Data)
	{
		Character = Character == From ? To : Character;
	}
}

void UE_STRING_CLASS::ReplaceCharInlineIgnoreCase(const ElementType From, const ElementType To)
{
	ElementType OtherCaseFrom = TChar<ElementType>::IsUpper(From) ? TChar<ElementType>::ToLower(From) : TChar<ElementType>::ToUpper(From);
	ReplaceCharInlineCaseSensitive(OtherCaseFrom, To);
	ReplaceCharInlineCaseSensitive(From, To);
}

void UE_STRING_CLASS::TrimStartAndEndInline()
{
	TrimEndInline();
	TrimStartInline();
}

UE_STRING_CLASS UE_STRING_CLASS::TrimStartAndEnd() const &
{
	UE_STRING_CLASS Result(*this);
	Result.TrimStartAndEndInline();
	return Result;
}

UE_STRING_CLASS UE_STRING_CLASS::TrimStartAndEnd() &&
{
	TrimStartAndEndInline();
	return MoveTemp(*this);
}

void UE_STRING_CLASS::TrimStartInline()
{
	int32 Pos = 0;
	while(Pos < Len() && TChar<ElementType>::IsWhitespace((*this)[Pos]))
	{
		Pos++;
	}
	RemoveAt(0, Pos);
}

UE_STRING_CLASS UE_STRING_CLASS::TrimStart() const &
{
	UE_STRING_CLASS Result(*this);
	Result.TrimStartInline();
	return Result;
}

UE_STRING_CLASS UE_STRING_CLASS::TrimStart() &&
{
	TrimStartInline();
	return MoveTemp(*this);
}

void UE_STRING_CLASS::TrimEndInline()
{
	int32 End = Len();
	while(End > 0 && TChar<ElementType>::IsWhitespace((*this)[End - 1]))
	{
		End--;
	}
	RemoveAt(End, Len() - End);
}

UE_STRING_CLASS UE_STRING_CLASS::TrimEnd() const &
{
	UE_STRING_CLASS Result(*this);
	Result.TrimEndInline();
	return Result;
}

UE_STRING_CLASS UE_STRING_CLASS::TrimEnd() &&
{
	TrimEndInline();
	return MoveTemp(*this);
}

void UE_STRING_CLASS::TrimCharInline(const ElementType CharacterToTrim, bool* bCharRemoved)
{
	bool bQuotesWereRemoved=false;
	int32 Start = 0, Count = Len();
	if (Count > 0)
	{
		if ((*this)[0] == CharacterToTrim)
		{
			Start++;
			Count--;
			bQuotesWereRemoved=true;
		}

		if (Len() > 1 && (*this)[Len() - 1] == CharacterToTrim)
		{
			Count--;
			bQuotesWereRemoved=true;
		}
	}

	if (bCharRemoved != nullptr)
	{
		*bCharRemoved = bQuotesWereRemoved;
	}
	MidInline(Start, Count, EAllowShrinking::No);
}

void UE_STRING_CLASS::TrimQuotesInline(bool* bQuotesRemoved)
{
	TrimCharInline(ElementType('"'), bQuotesRemoved);
}

UE_STRING_CLASS UE_STRING_CLASS::TrimQuotes(bool* bQuotesRemoved) const &
{
	UE_STRING_CLASS Result(*this);
	Result.TrimQuotesInline(bQuotesRemoved);
	return Result;
}

UE_STRING_CLASS UE_STRING_CLASS::TrimQuotes(bool* bQuotesRemoved) &&
{
	TrimQuotesInline(bQuotesRemoved);
	return MoveTemp(*this);
}

UE_STRING_CLASS UE_STRING_CLASS::TrimChar(const ElementType CharacterToTrim, bool* bCharRemoved) const &
{
	UE_STRING_CLASS Result(*this);
	Result.TrimCharInline(CharacterToTrim, bCharRemoved);
	return Result;
}

UE_STRING_CLASS UE_STRING_CLASS::TrimChar(const ElementType CharacterToTrim, bool* bCharRemoved) &&
{
	TrimCharInline(CharacterToTrim, bCharRemoved);
	return MoveTemp(*this);
}

int32 UE_STRING_CLASS::CullArray(TArray<UE_STRING_CLASS>* InOutArray)
{
	check(InOutArray);
	UE_STRING_CLASS Empty;
	InOutArray->Remove(Empty);
	return InOutArray->Num();
}

UE_STRING_CLASS UE_STRING_CLASS::Reverse() const &
{
	UE_STRING_CLASS New(*this);
	New.ReverseString();
	return New;
}

UE_STRING_CLASS UE_STRING_CLASS::Reverse() &&
{
	ReverseString();
	return MoveTemp(*this);
}

void UE_STRING_CLASS::ReverseString()
{
	if (Len() > 0)
	{
		ElementType* StartChar = &(*this)[0];
		ElementType* EndChar = &(*this)[Len()-1];
		ElementType TempChar;
		do 
		{
			TempChar = *StartChar;	// store the current value of StartChar
			*StartChar = *EndChar;	// change the value of StartChar to the value of EndChar
			*EndChar = TempChar;	// change the value of EndChar to the character that was previously at StartChar

			StartChar++;
			EndChar--;

		} while (StartChar < EndChar);	// repeat until we've reached the midpoint of the string
	}
}

UE_STRING_CLASS UE_STRING_CLASS::FormatAsNumber(int32 InNumber)
{
	UE_STRING_CLASS Number = UE_STRING_CLASS::FromInt(InNumber), Result;

	int32 Dec = 0;
	for (int32 x = Number.Len()-1 ; x > -1 ; --x)
	{
		Result += Number.Mid(x,1);

		Dec++;
		if (Dec == 3 && x > 0)
		{
			Result += CHARTEXT(ElementType, ',');
			Dec = 0;
		}
	}

	return Result.Reverse();
}

/**
 * Serializes a string as ANSI char array.
 *
 * @param	String			String to serialize
 * @param	Ar				Archive to serialize with
 * @param	MinCharacters	Minimum number of characters to serialize.
 */
void UE_STRING_CLASS::SerializeAsANSICharArray(FArchive& Ar, int32 MinCharacters) const
{
	int32 Length = FMath::Max(Len(), MinCharacters);
	Ar << Length;

	for (int32 CharIndex=0; CharIndex<Len(); CharIndex++)
	{
		ANSICHAR AnsiChar = CharCast<ANSICHAR>((*this)[CharIndex]);
		Ar << AnsiChar;
	}

	// Zero pad till minimum number of characters are written.
	for (int32 i = Len(); i < Length; i++)
	{
		ANSICHAR NullChar = 0;
		Ar << NullChar;
	}
}

void UE_STRING_CLASS::AppendInt(int32 Num)
{
	const ElementType* DigitToChar	= CHARTEXT(ElementType, "9876543210123456789");
	constexpr int32 ZeroDigitIndex	= 9;
	bool bIsNumberNegative			= Num < 0;
	const int32 TempBufferSize		= 16; // 16 is big enough
	ElementType TempNum[TempBufferSize];
	int32 TempAt					= TempBufferSize; // fill the temp string from the top down.

	// Convert to string assuming base ten.
	do 
	{
		TempNum[--TempAt] = DigitToChar[ZeroDigitIndex + (Num % 10)];
		Num /= 10;
	} while (Num);

	if (bIsNumberNegative)
	{
		TempNum[--TempAt] = CHARTEXT(ElementType, '-');
	}

	const ElementType* CharPtr = TempNum + TempAt;
	const int32 NumChars = TempBufferSize - TempAt;
	Append(CharPtr, NumChars);
}


bool UE_STRING_CLASS::ToBool() const
{
	return TCString<ElementType>::ToBool(**this);
}

UE_STRING_CLASS UE_STRING_CLASS::FromBlob(const uint8* SrcBuffer, const uint32 SrcSize)
{
	UE_STRING_CLASS Result;
	Result.Reserve(SrcSize * 3);
	// Convert and append each byte in the buffer
	for (uint32 Count = 0; Count < SrcSize; Count++)
	{
		Result += UE_STRING_CLASS::Printf(CHARTEXT(FmtCharType, "%03d"),(uint8)SrcBuffer[Count]);
	}
	return Result;
}

bool UE_STRING_CLASS::ToBlob(const UE_STRING_CLASS& Source, uint8* DestBuffer, const uint32 DestSize)
{
	// Make sure the buffer is at least half the size and that the string is an
	// even number of characters long
	if (DestSize >= (uint32)(Source.Len() / 3) &&
		(Source.Len() % 3) == 0)
	{
		ElementType ConvBuffer[4];
		ConvBuffer[3] = CHARTEXT(ElementType, '\0');
		int32 WriteIndex = 0;
		// Walk the string 3 chars at a time
		for (int32 Index = 0; Index < Source.Len(); Index += 3, WriteIndex++)
		{
			ConvBuffer[0] = Source[Index];
			ConvBuffer[1] = Source[Index + 1];
			ConvBuffer[2] = Source[Index + 2];
			DestBuffer[WriteIndex] = (uint8)TCString<ElementType>::Atoi(ConvBuffer);
		}
		return true;
	}
	return false;
}

UE_STRING_CLASS UE_STRING_CLASS::FromHexBlob(const uint8* SrcBuffer, const uint32 SrcSize)
{
	UE_STRING_CLASS Result;
	Result.Reserve(SrcSize * 2);
	// Convert and append each byte in the buffer
	for (uint32 Count = 0; Count < SrcSize; Count++)
	{
		Result += UE_STRING_CLASS::Printf(CHARTEXT(FmtCharType, "%02X"), (uint8)SrcBuffer[Count]);
	}
	return Result;
}

bool UE_STRING_CLASS::ToHexBlob(const UE_STRING_CLASS& Source, uint8* DestBuffer, const uint32 DestSize)
{
	// Make sure the buffer is at least half the size and that the string is an
	// even number of characters long
	if (DestSize >= (uint32)(Source.Len() / 2) &&
		 (Source.Len() % 2) == 0)
	{
		ElementType ConvBuffer[3];
		ConvBuffer[2] = CHARTEXT(ElementType, '\0');
		int32 WriteIndex = 0;
		// Walk the string 2 chars at a time
		ElementType* End = nullptr;
		for (int32 Index = 0; Index < Source.Len(); Index += 2, WriteIndex++)
		{
			ConvBuffer[0] = Source[Index];
			ConvBuffer[1] = Source[Index + 1];
			DestBuffer[WriteIndex] = (uint8)TCString<ElementType>::Strtoi(ConvBuffer, &End, 16);
		}
		return true;
	}
	return false;
}

UE_STRING_CLASS UE_STRING_CLASS::SanitizeFloat(double InFloat, const int32 InMinFractionalDigits)
{
	// Avoids negative zero
	UE::Core::Private::StripNegativeZero(InFloat);

	// First create the string
	UE_STRING_CLASS TempString = UE_STRING_CLASS::Printf(CHARTEXT(FmtCharType, "%f"), InFloat);
	if (!TempString.IsNumeric())
	{
		// String did not format as a valid decimal number so avoid messing with it
		return TempString;
	}

	// Trim all trailing zeros (up-to and including the decimal separator) from the fractional part of the number
	int32 TrimIndex = INDEX_NONE;
	int32 DecimalSeparatorIndex = INDEX_NONE;
	for (int32 CharIndex = TempString.Len() - 1; CharIndex >= 0; --CharIndex)
	{
		const ElementType Char = TempString[CharIndex];
		if (Char == CHARTEXT(ElementType, '.'))
		{
			DecimalSeparatorIndex = CharIndex;
			TrimIndex = FMath::Max(TrimIndex, DecimalSeparatorIndex);
			break;
		}
		if (TrimIndex == INDEX_NONE && Char != CHARTEXT(ElementType, '0'))
		{
			TrimIndex = CharIndex + 1;
		}
	}
	check(TrimIndex != INDEX_NONE && DecimalSeparatorIndex != INDEX_NONE);
	TempString.RemoveAt(TrimIndex, TempString.Len() - TrimIndex, EAllowShrinking::No);

	// Pad the number back to the minimum number of fractional digits
	if (InMinFractionalDigits > 0)
	{
		if (TrimIndex == DecimalSeparatorIndex)
		{
			// Re-add the decimal separator
			TempString.AppendChar(CHARTEXT(ElementType, '.'));
		}

		const int32 NumFractionalDigits = (TempString.Len() - DecimalSeparatorIndex) - 1;
		const int32 FractionalDigitsToPad = InMinFractionalDigits - NumFractionalDigits;
		if (FractionalDigitsToPad > 0)
		{
			TempString.Reserve(TempString.Len() + FractionalDigitsToPad);
			for (int32 Cx = 0; Cx < FractionalDigitsToPad; ++Cx)
			{
				TempString.AppendChar(CHARTEXT(ElementType, '0'));
			}
		}
	}

	return TempString;
}

UE_STRING_CLASS UE_STRING_CLASS::Chr(ElementType Ch)
{
	ElementType Temp[2]= { Ch, CHARTEXT(ElementType, '\0') };
	return UE_STRING_CLASS(Temp);
}


UE_STRING_CLASS UE_STRING_CLASS::ChrN(int32 NumCharacters, ElementType Char)
{
	check(NumCharacters >= 0);

	UE_STRING_CLASS Temp;
	Temp.Data.AddUninitialized(NumCharacters+1);
	for (int32 Cx = 0; Cx < NumCharacters; ++Cx)
	{
		Temp[Cx] = Char;
	}
	Temp.Data[NumCharacters] = CHARTEXT(ElementType, '\0');
	return Temp;
}

UE_STRING_CLASS UE_STRING_CLASS::LeftPad(int32 ChCount) const
{
	int32 Pad = ChCount - Len();

	if (Pad > 0)
	{
		return ChrN(Pad, CHARTEXT(ElementType, ' ')) + *this;
	}
	else
	{
		return *this;
	}
}
UE_STRING_CLASS UE_STRING_CLASS::RightPad(int32 ChCount) const
{
	int32 Pad = ChCount - Len();

	if (Pad > 0)
	{
		return *this + ChrN(Pad, CHARTEXT(ElementType, ' '));
	}
	else
	{
		return *this;
	}
}

bool UE_STRING_CLASS::IsNumeric() const
{
	if (IsEmpty())
	{
		return 0;
	}

	return TCString<ElementType>::IsNumeric(Data.GetData());
}

int32 UE_STRING_CLASS::ParseIntoArray(TArray<UE_STRING_CLASS>& OutArray, const ElementType* pchDelim, const bool bInCullEmpty) const
{
	check(pchDelim);
	OutArray.Reset();
	// TODO Legacy behavior: if DelimLength is 0 we return an empty array. Can we change this to return { Text }?
	if (pchDelim[0] != '\0')
	{
		UE::String::EParseTokensOptions ParseOptions = UE::String::EParseTokensOptions::IgnoreCase |
			(bInCullEmpty ? UE::String::EParseTokensOptions::SkipEmpty : UE::String::EParseTokensOptions::None);
		UE::String::ParseTokens(TStringView<ElementType>(*this), TStringView<ElementType>(pchDelim),
			[&OutArray](TStringView<ElementType> Token) { OutArray.Emplace(Token); },
			ParseOptions);
	}
	return OutArray.Num();
}

bool UE_STRING_CLASS::MatchesWildcard(const ElementType* InWildcard, int32 InWildcardLen, ESearchCase::Type SearchCase) const
{
	const ElementType* Target = **this;
	int32        TargetLength = Len();

	if (SearchCase == ESearchCase::CaseSensitive)
	{
		return UE::Core::Private::MatchesWildcardRecursive(
			Target,
			TargetLength,
			InWildcard,
			InWildcardLen,
			[](UE_STRING_CHARTYPE Lhs, UE_STRING_CHARTYPE Rhs)
			{
				return Lhs == Rhs;
			}
		);
	}
	else
	{
		return UE::Core::Private::MatchesWildcardRecursive(
			Target,
			TargetLength,
			InWildcard,
			InWildcardLen,
			[](UE_STRING_CHARTYPE Lhs, UE_STRING_CHARTYPE Rhs)
			{
				return TChar<UE_STRING_CHARTYPE>::ToLower(Lhs) == TChar<UE_STRING_CHARTYPE>::ToLower(Rhs);
			}
		);
	}
}


/** Caution!! this routine is O(N^2) allocations...use it for parsing very short text or not at all */
int32 UE_STRING_CLASS::ParseIntoArrayWS(TArray<UE_STRING_CLASS>& OutArray, const ElementType* pchExtraDelim, bool bInCullEmpty) const
{
	// default array of White Spaces, the last entry can be replaced with the optional pchExtraDelim string
	// (if you want to split on white space and another character)
	const ElementType* WhiteSpace[] =
	{
		CHARTEXT(ElementType, " "),
		CHARTEXT(ElementType, "\t"),
		CHARTEXT(ElementType, "\r"),
		CHARTEXT(ElementType, "\n"),
		CHARTEXT(ElementType, ""),
	};

	// start with just the standard whitespaces
	int32 NumWhiteSpaces = UE_ARRAY_COUNT(WhiteSpace) - 1;
	// if we got one passed in, use that in addition
	if (pchExtraDelim && *pchExtraDelim)
	{
		WhiteSpace[NumWhiteSpaces++] = pchExtraDelim;
	}

	return ParseIntoArray(OutArray, WhiteSpace, NumWhiteSpaces, bInCullEmpty);
}

int32 UE_STRING_CLASS::ParseIntoArrayLines(TArray<UE_STRING_CLASS>& OutArray, bool bInCullEmpty) const
{
	// default array of LineEndings
	static const ElementType* LineEndings[] =
	{
		CHARTEXT(ElementType, "\r\n"),
		CHARTEXT(ElementType, "\r"),
		CHARTEXT(ElementType, "\n"),
	};

	// start with just the standard line endings
	int32 NumLineEndings = UE_ARRAY_COUNT(LineEndings);	
	return ParseIntoArray(OutArray, LineEndings, NumLineEndings, bInCullEmpty);
}

int32 UE_STRING_CLASS::ParseIntoArray(TArray<UE_STRING_CLASS>& OutArray, const ElementType* const * DelimArray, int32 NumDelims, bool bInCullEmpty) const
{
	// Make sure the delimit string is not null or empty
	check(DelimArray);
	OutArray.Reset();
	const ElementType* Start = Data.GetData();
	const int32 Length = Len();
	int32 SubstringBeginIndex = 0;

		// Iterate through string.
	for(int32 i = 0; i < Length;)
	{
		int32 SubstringEndIndex = INDEX_NONE;
		int32 DelimiterLength = 0;

		// Attempt each delimiter.
		for(int32 DelimIndex = 0; DelimIndex < NumDelims; ++DelimIndex)
		{
			DelimiterLength = TCString<ElementType>::Strlen(DelimArray[DelimIndex]);

			// If we found a delimiter...
			if (TCString<ElementType>::Strncmp(Start + i, DelimArray[DelimIndex], DelimiterLength) == 0)
			{
				// Mark the end of the substring.
				SubstringEndIndex = i;
				break;
			}
		}

		if (SubstringEndIndex != INDEX_NONE)
		{
			const int32 SubstringLength = SubstringEndIndex - SubstringBeginIndex;
			// If we're not culling empty strings or if we are but the string isn't empty anyways...
			if(!bInCullEmpty || SubstringLength != 0)
			{
				// ... add new string from substring beginning up to the beginning of this delimiter.
				OutArray.Add(UE_STRING_CLASS::ConstructFromPtrSize(Start + SubstringBeginIndex, SubstringEndIndex - SubstringBeginIndex));
			}
			// Next substring begins at the end of the discovered delimiter.
			SubstringBeginIndex = SubstringEndIndex + DelimiterLength;
			i = SubstringBeginIndex;
		}
		else
		{
			++i;
		}
	}

	// Add any remaining characters after the last delimiter.
	const int32 SubstringLength = Length - SubstringBeginIndex;
	// If we're not culling empty strings or if we are but the string isn't empty anyways...
	if(!bInCullEmpty || SubstringLength != 0)
	{
		// ... add new string from substring beginning up to the beginning of this delimiter.
		OutArray.Emplace(TStringView<ElementType>(Start + SubstringBeginIndex, SubstringLength));
	}

	return OutArray.Num();
}

UE_STRING_CLASS UE_STRING_CLASS::Replace(const ElementType* From, const ElementType* To, ESearchCase::Type SearchCase) const &
{
	// Previous code used to accidentally accept a nullptr replacement string - this is no longer accepted.
	check(To);

	if (IsEmpty() || !From || !*From)
	{
		return *this;
	}

	// get a pointer into the character data
	const ElementType* Travel = Data.GetData();

	// precalc the lengths of the replacement strings
	int32 FromLength = TCString<ElementType>::Strlen(From);
	int32 ToLength   = TCString<ElementType>::Strlen(To);

	UE_STRING_CLASS Result;
	while (true)
	{
		// look for From in the remaining string
		const ElementType* FromLocation = SearchCase == ESearchCase::IgnoreCase ? TCString<ElementType>::Stristr(Travel, From) : TCString<ElementType>::Strstr(Travel, From);
		if (!FromLocation)
		{
			break;
		}

		// copy everything up to FromLocation
		Result.AppendChars(Travel, UE_PTRDIFF_TO_INT32(FromLocation - Travel));

		// copy over the To
		Result.AppendChars(To, ToLength);

		Travel = FromLocation + FromLength;
	}

	// copy anything left over
	Result += Travel;

	return Result;
}

UE_STRING_CLASS UE_STRING_CLASS::Replace(const ElementType* From, const ElementType* To, ESearchCase::Type SearchCase) &&
{
	ReplaceInline(From, To, SearchCase);
	return MoveTemp(*this);
}

int32 UE_STRING_CLASS::ReplaceInline(const ElementType* From, const ElementType* To, ESearchCase::Type SearchCase)
{
	int32 ReplacementCount = 0;

	if (Len() > 0
		&& From != nullptr && *From != 0
		&& To != nullptr && (SearchCase == ESearchCase::IgnoreCase || TCString<ElementType>::Strcmp(From, To) != 0))
	{
		const int32 NumCharsToReplace = TCString<ElementType>::Strlen(From);
		const int32 NumCharsToInsert = TCString<ElementType>::Strlen(To);

		if (NumCharsToInsert == NumCharsToReplace)
		{
			ElementType* Pos = SearchCase == ESearchCase::IgnoreCase ? TCString<ElementType>::Stristr(&(*this)[0], From) : TCString<ElementType>::Strstr(&(*this)[0], From);
			while (Pos != nullptr)
			{
				ReplacementCount++;

				// TCString<ElementType>::Strcpy now inserts a terminating zero so can't use that
				for (int32 i = 0; i < NumCharsToInsert; i++)
				{
					Pos[i] = To[i];
				}

				if (Pos + NumCharsToReplace - **this < Len())
				{
					Pos = SearchCase == ESearchCase::IgnoreCase ? TCString<ElementType>::Stristr(Pos + NumCharsToReplace, From) : TCString<ElementType>::Strstr(Pos + NumCharsToReplace, From);
				}
				else
				{
					break;
				}
			}
		}
		else if (Contains(From, SearchCase))
		{
			UE_STRING_CLASS Copy(MoveTemp(*this));

			// get a pointer into the character data
			ElementType* WritePosition = (ElementType*)Copy.Data.GetData();
			// look for From in the remaining string
			ElementType* SearchPosition = SearchCase == ESearchCase::IgnoreCase ? TCString<ElementType>::Stristr(WritePosition, From) : TCString<ElementType>::Strstr(WritePosition, From);
			while (SearchPosition != nullptr)
			{
				ReplacementCount++;

				// replace the first letter of the From with 0 so we can do a strcpy (via operator+=)
				*SearchPosition = CHARTEXT(ElementType, '\0');

				// copy everything up to the SearchPosition
				(*this) += WritePosition;

				// copy over the replacement string
				(*this) += To;

				// restore the letter, just so we don't have 0's in the string
				*SearchPosition = *From;

				WritePosition = SearchPosition + NumCharsToReplace;
				SearchPosition = SearchCase == ESearchCase::IgnoreCase ? TCString<ElementType>::Stristr(WritePosition, From) : TCString<ElementType>::Strstr(WritePosition, From);
			}

			// copy anything left over
			(*this) += WritePosition;
		}
	}

	return ReplacementCount;
}


/**
 * Returns a copy of this string with all quote marks escaped (unless the quote is already escaped)
 */
UE_STRING_CLASS UE_STRING_CLASS::ReplaceQuotesWithEscapedQuotes() &&
{
	if (Contains(CHARTEXT(ElementType, "\""), ESearchCase::CaseSensitive))
	{
		UE_STRING_CLASS Copy(MoveTemp(*this));

		const ElementType* pChar = *Copy;

		bool bEscaped = false;
		while (*pChar != 0)
		{
			if (bEscaped)
			{
				bEscaped = false;
			}
			else if (*pChar == ElementType('\\'))
			{
				bEscaped = true;
			}
			else if (*pChar == ElementType('"'))
			{
				*this += ElementType('\\');
			}

			*this += *pChar++;
		}
	}

	return MoveTemp(*this);
}

static const UE_STRING_CHARTYPE* UE_JOIN(CharToEscapeSeqMap_, UE_STRING_CLASS)[6][2] =
{
	// Always replace \\ first to avoid double-escaping characters
	{ CHARTEXT(UE_STRING_CHARTYPE, "\\"), CHARTEXT(UE_STRING_CHARTYPE, "\\\\") },
	{ CHARTEXT(UE_STRING_CHARTYPE, "\n"), CHARTEXT(UE_STRING_CHARTYPE, "\\n")  },
	{ CHARTEXT(UE_STRING_CHARTYPE, "\r"), CHARTEXT(UE_STRING_CHARTYPE, "\\r")  },
	{ CHARTEXT(UE_STRING_CHARTYPE, "\t"), CHARTEXT(UE_STRING_CHARTYPE, "\\t")  },
	{ CHARTEXT(UE_STRING_CHARTYPE, "\'"), CHARTEXT(UE_STRING_CHARTYPE, "\\'")  },
	{ CHARTEXT(UE_STRING_CHARTYPE, "\""), CHARTEXT(UE_STRING_CHARTYPE, "\\\"") }
};

void UE_STRING_CLASS::ReplaceCharWithEscapedCharInline(const TArray<ElementType>* Chars/*=nullptr*/)
{
	if (Len() > 0 && (Chars == nullptr || Chars->Num() > 0))
	{
		const auto& CharToEscapeSeqMap = UE_JOIN(CharToEscapeSeqMap_, UE_STRING_CLASS);

		for (uint32 ChIdx = 0; ChIdx < UE_ARRAY_COUNT(CharToEscapeSeqMap); ChIdx++)
		{
			if (Chars == nullptr || Chars->Contains(*(CharToEscapeSeqMap[ChIdx][0])))
			{
				// use ReplaceInline as that won't create a copy of the string if the character isn't found
				ReplaceInline(CharToEscapeSeqMap[ChIdx][0], CharToEscapeSeqMap[ChIdx][1]);
			}
		}
	}
}

void UE_STRING_CLASS::ReplaceEscapedCharWithCharInline(const TArray<ElementType>* Chars/*=nullptr*/)
{
	if (Len() > 0 && (Chars == nullptr || Chars->Num() > 0))
	{
		const auto& CharToEscapeSeqMap = UE_JOIN(CharToEscapeSeqMap_, UE_STRING_CLASS);

		// Spin CharToEscapeSeqMap backwards to ensure we're doing the inverse of ReplaceCharWithEscapedChar
		for (int32 ChIdx = UE_ARRAY_COUNT(CharToEscapeSeqMap); ChIdx > 0; )
		{
			--ChIdx;

			if (Chars == nullptr || Chars->Contains(*(CharToEscapeSeqMap[ChIdx][0])))
			{
				// use ReplaceInline as that won't create a copy of the string if the character isn't found
				ReplaceInline(CharToEscapeSeqMap[ChIdx][1], CharToEscapeSeqMap[ChIdx][0]);
			}
		}
	}
}

/** 
 * Replaces all instances of '\t' with TabWidth number of spaces
 * @param InSpacesPerTab - Number of spaces that a tab represents
 */
void UE_STRING_CLASS::ConvertTabsToSpacesInline(const int32 InSpacesPerTab)
{
	//must call this with at least 1 space so the modulus operation works
	check(InSpacesPerTab > 0);

	int32 TabIndex = 0;
	while ((TabIndex = Find(CHARTEXT(ElementType, "\t"), ESearchCase::CaseSensitive)) != INDEX_NONE)
	{
		UE_STRING_CLASS RightSide = Mid(TabIndex+1);
		LeftInline(TabIndex, EAllowShrinking::No);

		//for a tab size of 4, 
		int32 LineBegin = Find(CHARTEXT(ElementType, "\n"), ESearchCase::CaseSensitive, ESearchDir::FromEnd, TabIndex);
		if (LineBegin == INDEX_NONE)
		{
			LineBegin = 0;
		}
		const int32 CharactersOnLine = (Len()-LineBegin);

		int32 NumSpacesForTab = InSpacesPerTab - (CharactersOnLine % InSpacesPerTab);
		for (int32 i = 0; i < NumSpacesForTab; ++i)
		{
			AppendChar(CHARTEXT(ElementType, ' '));
		}
		Append(RightSide);
	}
}

// This starting size catches 99.97% of printf calls - there are about 700k printf calls per level
#define STARTING_BUFFER_SIZE		512

UE_STRING_CLASS UE_STRING_CLASS::PrintfImpl(const ElementType* Fmt, ...)
{
	int32		BufferSize	= STARTING_BUFFER_SIZE;
	ElementType	StartingBuffer[STARTING_BUFFER_SIZE];
	ElementType*	Buffer		= StartingBuffer;
	int32		Result		= -1;

	// First try to print to a stack allocated location 
	GET_TYPED_VARARGS_RESULT(ElementType, Buffer, BufferSize, BufferSize-1, Fmt, Fmt, Result);

	// If that fails, start allocating regular memory
	if (Result == -1)
	{
		Buffer = nullptr;
		while(Result == -1)
		{
			BufferSize *= 2;
			Buffer = (ElementType*) FMemory::Realloc(Buffer, BufferSize * sizeof(ElementType));
			GET_TYPED_VARARGS_RESULT(ElementType, Buffer, BufferSize, BufferSize-1, Fmt, Fmt, Result);
		};
	}

	Buffer[Result] = CHARTEXT(ElementType, '\0');

	UE_STRING_CLASS ResultString(Buffer);

	if (BufferSize != STARTING_BUFFER_SIZE)
	{
		FMemory::Free(Buffer);
	}

	return ResultString;
}

void UE_STRING_CLASS::AppendfImpl(UE_STRING_CLASS& AppendToMe, const ElementType* Fmt, ...)
{
	int32		BufferSize = STARTING_BUFFER_SIZE;
	ElementType	StartingBuffer[STARTING_BUFFER_SIZE];
	ElementType*	Buffer = StartingBuffer;
	int32		Result = -1;

	// First try to print to a stack allocated location 
	GET_TYPED_VARARGS_RESULT(ElementType, Buffer, BufferSize, BufferSize - 1, Fmt, Fmt, Result);

	// If that fails, start allocating regular memory
	if (Result == -1)
	{
		Buffer = nullptr;
		while (Result == -1)
		{
			BufferSize *= 2;
			Buffer = (ElementType*)FMemory::Realloc(Buffer, BufferSize * sizeof(ElementType));
			GET_TYPED_VARARGS_RESULT(ElementType, Buffer, BufferSize, BufferSize - 1, Fmt, Fmt, Result);
		};
	}

	Buffer[Result] = CHARTEXT(ElementType, '\0');

	AppendToMe += Buffer;

	if (BufferSize != STARTING_BUFFER_SIZE)
	{
		FMemory::Free(Buffer);
	}
}

static_assert(PLATFORM_LITTLE_ENDIAN, UE_STRINGIZE(UE_STRING_CLASS) " serialization needs updating to support big-endian platforms!");

FArchive& operator<<(FArchive& Ar, UE_STRING_CLASS& Str)
{
	using ElementType = UE_STRING_CLASS::ElementType;

	#if UE_STRING_CHARTYPE_IS_TCHAR
		// > 0 for ANSICHAR, < 0 for UTF16CHAR serialization
		static_assert(sizeof(UTF16CHAR) == sizeof(UCS2CHAR), "UTF16CHAR and UCS2CHAR are assumed to be the same size!");
	#endif

	if (Ar.IsLoading())
	{
		int32 SaveNum = 0;
		Ar << SaveNum;

		// Validate loaded num to ensure the archive is in a good state
		#if UE_STRING_CHARTYPE_IS_TCHAR
			bool bLoadUnicodeChar = SaveNum < 0;
			if (bLoadUnicodeChar)
			{
				// If SaveNum cannot be negated due to integer overflow, Ar is corrupted.
				if (SaveNum == MIN_int32)
				{
					Ar.SetCriticalError();
					UE_LOGF(LogCore, Error, "Archive is corrupted");
					return Ar;
				}

				SaveNum = -SaveNum;
			}
		#else
			if (SaveNum < 0)
			{
				Ar.SetCriticalError();
				UE_LOGF(LogCore, Error, "Archive is corrupted");
				return Ar;
			}
		#endif

		int64 MaxSerializeSize = Ar.GetMaxSerializeSize();
		// Protect against network packets allocating too much memory
		if ((MaxSerializeSize > 0) && (SaveNum > MaxSerializeSize))
		{
			Ar.SetCriticalError();
			UE_LOG(LogCore, Error, TEXT("String is too large (Size: %i, Max: %" INT64_FMT ")"), SaveNum, MaxSerializeSize);
			return Ar;
		}

		#if UE_STRING_CHARTYPE_IS_TCHAR
			// Resize the array only if it passes the above tests to prevent rogue packets from crashing
			Str.Data.Empty(SaveNum);
			Str.Data.AddUninitialized(SaveNum);

			if (SaveNum)
			{
				if (bLoadUnicodeChar)
				{
					// read in the unicode string
					auto Passthru = StringMemoryPassthru<UCS2CHAR>(Str.Data.GetData(), SaveNum, SaveNum);
					Ar.Serialize(Passthru.Get(), SaveNum * sizeof(UCS2CHAR));
					if (Ar.IsByteSwapping())
					{
						for (int32 CharIndex = 0; CharIndex < SaveNum; ++CharIndex)
						{
							Passthru.Get()[CharIndex] = ByteSwap(Passthru.Get()[CharIndex]);
						}
					}
					// Ensure the string has a null terminator
					Passthru.Get()[SaveNum - 1] = '\0';
					Passthru.Apply();

					// Since Microsoft's vsnwprintf implementation raises an invalid parameter warning
					// with a character of 0xffff, scan for it and terminate the string there.
					// 0xffff isn't an actual Unicode character anyway.
					int Index = 0;
					if (Str.FindChar(0xffff, Index))
					{
						Str[Index] = CHARTEXT(ElementType, '\0');
						Str.TrimToNullTerminator();
					}
				}
				else
				{
					auto Passthru = StringMemoryPassthru<ANSICHAR>(Str.Data.GetData(), SaveNum, SaveNum);
					Ar.Serialize(Passthru.Get(), SaveNum * sizeof(ANSICHAR));
					// Ensure the string has a null terminator
					Passthru.Get()[SaveNum - 1] = '\0';
					Passthru.Apply();
				}

				// Throw away empty string.
				if (SaveNum == 1)
				{
					Str.Data.Empty();
				}
			}
		#else
			if (SaveNum)
			{
				Str.Data.Empty(SaveNum + 1);
				Str.Data.AddUninitialized(SaveNum + 1);

				auto Passthru = StringMemoryPassthru<ElementType>(Str.Data.GetData(), SaveNum, SaveNum);
				Ar.Serialize(Passthru.Get(), SaveNum * sizeof(ElementType));
				// Ensure the string has a null terminator
				Passthru.Get()[SaveNum] = UTF8TEXT('\0');
				Passthru.Apply();

				// We don't need to throw away empty strings here, unlike above, because we never saved a null terminator
			}
			else
			{
				Str.Empty();
			}
		#endif
	}
	else
	{
		Str.Data.CountBytes(Ar);

		#if UE_STRING_CHARTYPE_IS_TCHAR
			const bool bSaveUnicodeChar = Ar.IsForcingUnicode() || !TCString<ElementType>::IsPureAnsi(*Str);
			if (bSaveUnicodeChar)
			{
				// This preprocessor block should not be necessary when the StrCast below understands UTF16CHAR.
				FTCHARToUTF16 UTF16String(*Str, Str.Len() + 1); // include the null terminator
				int32 Num = UTF16String.Length() + 1; // include the null terminator

				int32 SaveNum = -Num;
				Ar << SaveNum;

				if (Num)
				{
					if (!Ar.IsByteSwapping())
					{
						Ar.Serialize((void*)UTF16String.Get(), sizeof(UTF16CHAR) * Num);
					}
					else
					{
						TArray<UTF16CHAR> Swapped(UTF16String.Get(), Num);
						for (int32 CharIndex = 0; CharIndex < Num; ++CharIndex)
						{
							Swapped[CharIndex] = ByteSwap(Swapped[CharIndex]);
						}
						Ar.Serialize((void*)Swapped.GetData(), sizeof(UTF16CHAR) * Num);
					}
				}
			}
			else
			{
				int32 Num = Str.Data.Num();
				Ar << Num;

				if (Num)
				{
					Ar.Serialize((void*)StrCast<ANSICHAR>(Str.Data.GetData(), Num).Get(), sizeof(ANSICHAR) * Num);
				}
			}
		#else
			// Unlike the TCHAR case, we don't bother to save the null terminator
			int32 SaveNum = Str.Len();
			Ar << SaveNum;

			if (SaveNum)
			{
				auto CompactString = StrCast<UTF8CHAR>(Str.Data.GetData(), SaveNum);
				Ar.Serialize((void*)CompactString.Get(), sizeof(UTF8CHAR) * CompactString.Length());
			}
		#endif
	}

	return Ar;
}

int32 HexToBytes(const UE_STRING_CLASS& HexString, uint8* OutBytes)
{
	return UE::String::HexToBytes(HexString, OutBytes);
}

int32 FindMatchingClosingParenthesis(const UE_STRING_CLASS& TargetString, const int32 StartSearch)
{
	using ElementType = UE_STRING_CLASS::ElementType;

	check(StartSearch >= 0 && StartSearch <= TargetString.Len());// Check for usage, we do not accept INDEX_NONE like other string functions

	const ElementType* const StartPosition = (*TargetString) + StartSearch;
	const ElementType* CurrPosition = StartPosition;
	int32 ParenthesisCount = 0;

	// Move to first open parenthesis
	while (*CurrPosition != 0 && *CurrPosition != CHARTEXT(ElementType, '('))
	{
		++CurrPosition;
	}

	// Did we find the open parenthesis
	if (*CurrPosition == CHARTEXT(ElementType, '('))
	{
		++ParenthesisCount;
		++CurrPosition;

		while (*CurrPosition != 0 && ParenthesisCount > 0)
		{
			if (*CurrPosition == CHARTEXT(ElementType, '('))
			{
				++ParenthesisCount;
			}
			else if (*CurrPosition == CHARTEXT(ElementType, ')'))
			{
				--ParenthesisCount;
			}
			++CurrPosition;
		}

		// Did we find the matching close parenthesis
		if (ParenthesisCount == 0 && *(CurrPosition - 1) == CHARTEXT(ElementType, ')'))
		{
			return StartSearch + UE_PTRDIFF_TO_INT32((CurrPosition - 1) - StartPosition);
		}
	}

	return INDEX_NONE;
}

UE_STRING_CLASS SlugStringForValidName(const UE_STRING_CLASS& DisplayString, const UE_STRING_CLASS::ElementType* ReplaceWith /*= CHARTEXT(ElementType, "")*/)
{
	using ElementType = UE_STRING_CLASS::ElementType;

	UE_STRING_CLASS GeneratedName = DisplayString;

	// Convert the display label, which may consist of just about any possible character, into a
	// suitable name for a UObject (remove whitespace, certain symbols, etc.)
	{
		for (int32 BadCharacterIndex = 0; BadCharacterIndex < UE_ARRAY_COUNT(INVALID_OBJECTNAME_CHARACTERS) - 1; ++BadCharacterIndex)
		{
			const ElementType TestChar[2] = { (ElementType)INVALID_OBJECTNAME_CHARACTERS[BadCharacterIndex], CHARTEXT(ElementType, '\0') };
			const int32 NumReplacedChars = GeneratedName.ReplaceInline(TestChar, ReplaceWith);
		}
	}

	return GeneratedName;
}

#undef UE_INCLUDETOOL_IGNORE_INCONSISTENT_STATE
