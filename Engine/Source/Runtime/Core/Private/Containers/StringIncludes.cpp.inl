// Copyright Epic Games, Inc. All Rights Reserved.

/*******************************************************************************************************
 * NOTICE                                                                                              *
 *                                                                                                     *
 * This file is not intended to be included directly - it is only intended to contain the includes for *
 * String.cpp.inl.                                                                                     *
 *******************************************************************************************************/

#ifdef UE_STRING_CLASS
	#error "StringIncludes.cpp.inl should not be included after defining UE_STRING_CLASS"
#endif
#ifdef UE_STRING_CHARTYPE
	#error "StringIncludes.cpp.inl should not be included after defining UE_STRING_CHARTYPE"
#endif
#ifdef UE_STRING_CHARTYPE_IS_TCHAR
	#error "StringIncludes.cpp.inl should not be included after defining UE_STRING_CHARTYPE_IS_TCHAR"
#endif

#include "Containers/Array.h"
#include "Containers/StringConv.h"
#include "CoreGlobals.h"
#include "CoreTypes.h"
#include "HAL/PlatformString.h"
#include "HAL/UnrealMemory.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Math/NumericLimits.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/AssertionMacros.h"
#include "Misc/ByteSwap.h"
#include "Misc/CString.h"
#include "Misc/Char.h"
#include "Misc/VarArgs.h"
#include "Serialization/Archive.h"
#include "String/HexToBytes.h"
#include "String/ParseTokens.h"
#include "Templates/MemoryOps.h"
#include "Templates/RemoveReference.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/NameTypes.h"
