// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "Misc/AutomationTest.h"
#include "Misc/AssertionMacros.h"
#include "Misc/StringBuilder.h"
#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include "Internationalization/TextChar.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"

#if WITH_DEV_AUTOMATION_TESTS

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FStringSanitizeFloatTest, "System.Core.String.SanitizeFloat", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool FStringSanitizeFloatTest::RunTest(const FString& Parameters)
{
	auto DoTest = [this](const double InVal, const int32 InMinFractionalDigits, const FString& InExpected)
	{
		const FString Result = FString::SanitizeFloat(InVal, InMinFractionalDigits);
		if (!Result.Equals(InExpected, ESearchCase::CaseSensitive))
		{
			AddError(FString::Printf(TEXT("%f (%d digits) failure: result '%s' (expected '%s')"), InVal, InMinFractionalDigits, *Result, *InExpected));
		}
	};

	DoTest(+0.0, 0, TEXT("0"));
	DoTest(-0.0, 0, TEXT("0"));

	DoTest(+100.0000, 0, TEXT("100"));
	DoTest(+100.1000, 0, TEXT("100.1"));
	DoTest(+100.1010, 0, TEXT("100.101"));
	DoTest(-100.0000, 0, TEXT("-100"));
	DoTest(-100.1000, 0, TEXT("-100.1"));
	DoTest(-100.1010, 0, TEXT("-100.101"));

	DoTest(+100.0000, 1, TEXT("100.0"));
	DoTest(+100.1000, 1, TEXT("100.1"));
	DoTest(+100.1010, 1, TEXT("100.101"));
	DoTest(-100.0000, 1, TEXT("-100.0"));
	DoTest(-100.1000, 1, TEXT("-100.1"));
	DoTest(-100.1010, 1, TEXT("-100.101"));

	DoTest(+100.0000, 4, TEXT("100.0000"));
	DoTest(+100.1000, 4, TEXT("100.1000"));
	DoTest(+100.1010, 4, TEXT("100.1010"));
	DoTest(-100.0000, 4, TEXT("-100.0000"));
	DoTest(-100.1000, 4, TEXT("-100.1000"));
	DoTest(-100.1010, 4, TEXT("-100.1010"));

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FStringAppendIntTest, "System.Core.String.AppendInt", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool FStringAppendIntTest::RunTest(const FString& Parameters)
{
	auto DoTest = [this](const TCHAR* Call, const FString& Result, const TCHAR* InExpected)
	{
		if (!Result.Equals(InExpected, ESearchCase::CaseSensitive))
		{
			AddError(FString::Printf(TEXT("'%s' failure: result '%s' (expected '%s')"), Call, *Result, InExpected));
		}
	};

	{
		FString Zero;
		Zero.AppendInt(0);
		DoTest(TEXT("AppendInt(0)"), Zero, TEXT("0"));
	}

	{
		FString IntMin;
		IntMin.AppendInt(MIN_int32);
		DoTest(TEXT("AppendInt(MIN_int32)"), IntMin, TEXT("-2147483648"));
	}

	{
		FString IntMin;
		IntMin.AppendInt(MAX_int32);
		DoTest(TEXT("AppendInt(MAX_int32)"), IntMin, TEXT("2147483647"));
	}

	{
		FString AppendMultipleInts;
		AppendMultipleInts.AppendInt(1);
		AppendMultipleInts.AppendInt(-2);
		AppendMultipleInts.AppendInt(3);
		DoTest(TEXT("AppendInt(1);AppendInt(-2);AppendInt(3)"), AppendMultipleInts, TEXT("1-23"));
	}

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FStringUnicodeTest, "System.Core.String.Unicode", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool FStringUnicodeTest::RunTest(const FString& Parameters)
{
	auto DoTest = [this](const TCHAR* Call, const FString& Result, const TCHAR* InExpected)
	{
		if (!Result.Equals(InExpected, ESearchCase::CaseSensitive))
		{
			AddError(FString::Printf(TEXT("'%s' failure: result '%s' (expected '%s')"), Call, *Result, InExpected));
		}
	};

	// Test data used to verify basic processing of a Unicode character outside the BMP
	FString TestStr;
	if (FTextChar::ConvertCodepointToString(128512, TestStr))
	{
		// Verify that the string can be serialized and deserialized without losing any data
		{
			TArray<uint8> StringData;
			FString FromArchive = TestStr;

			FMemoryWriter Writer(StringData);
			Writer << FromArchive;

			FromArchive.Reset();
			FMemoryReader Reader(StringData);
			Reader << FromArchive;

			DoTest(TEXT("FromArchive"), FromArchive, *TestStr);
		}

		// Verify that the string can be converted from/to UTF-8 without losing any data
		{
			const FString FromUtf8 = UTF8_TO_TCHAR(TCHAR_TO_UTF8(*TestStr));
			DoTest(TEXT("FromUtf8"), FromUtf8, *TestStr);
		}

		// Verify that the string can be converted from/to UTF-16 without losing any data
		{
			const FString FromUtf16 = UTF16_TO_TCHAR(TCHAR_TO_UTF16(*TestStr));
			DoTest(TEXT("FromUtf16"), FromUtf16, *TestStr);
		}
	}

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FLexTryParseStringTest, "System.Core.Misc.LexTryParseString", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)

bool FLexTryParseStringTest::RunTest(const FString& Parameters)
{
	// Test that LexFromString can intepret all the numerical formats we expect it to
	{
		// Test float values

		float Value;

		// Basic numbers
		TestTrue(TEXT("(float conversion from string) basic numbers"), LexTryParseString(Value, (TEXT("1"))) && Value == 1);
		TestTrue(TEXT("(float conversion from string) basic numbers"), LexTryParseString(Value, (TEXT("1.0"))) && Value == 1);
		TestTrue(TEXT("(float conversion from string) basic numbers"), LexTryParseString(Value, (TEXT(".5"))) && Value == 0.5);
		TestTrue(TEXT("(float conversion from string) basic numbers"), LexTryParseString(Value, (TEXT("1."))) && Value == 1);

		// Variations of 0
		TestTrue(TEXT("(float conversion from string) variations of 0"), LexTryParseString(Value, (TEXT("0"))) && Value == 0);
		TestTrue(TEXT("(float conversion from string) variations of 0"), LexTryParseString(Value, (TEXT("-0"))) && Value == 0);
		TestTrue(TEXT("(float conversion from string) variations of 0"), LexTryParseString(Value, (TEXT("0.0"))) && Value == 0);
		TestTrue(TEXT("(float conversion from string) variations of 0"), LexTryParseString(Value, (TEXT(".0"))) && Value == 0);
		TestTrue(TEXT("(float conversion from string) variations of 0"), LexTryParseString(Value, (TEXT("0."))) && Value == 0);
		TestTrue(TEXT("(float conversion from string) variations of 0"), LexTryParseString(Value, (TEXT("0. 111"))) && Value == 0);

		// Scientific notation
		TestTrue(TEXT("(float conversion from string) scientific notation"), LexTryParseString(Value, (TEXT("1.0e+10"))) && Value == 1.0e+10f);
		TestTrue(TEXT("(float conversion from string) scientific notation"), LexTryParseString(Value, (TEXT("1.99999999e-11"))) && Value == 1.99999999e-11f);
		TestTrue(TEXT("(float conversion from string) scientific notation"), LexTryParseString(Value, (TEXT("1e+10"))) && Value == 1e+10f);

		// Non-finite special numbers
		TestTrue(TEXT("(float conversion from string) inf"), LexTryParseString(Value, (TEXT("inf"))));
		TestTrue(TEXT("(float conversion from string) nan"), LexTryParseString(Value, (TEXT("nan"))));
		TestTrue(TEXT("(float conversion from string) nan(ind)"), LexTryParseString(Value, (TEXT("nan(ind)"))));

		// nan/inf etc. are detected from the start of the string, regardless of any other characters that come afterwards
		TestTrue(TEXT("(float conversion from string) nananananananana"), LexTryParseString(Value, (TEXT("nananananananana"))));
		TestTrue(TEXT("(float conversion from string) nan(ind)!"), LexTryParseString(Value, (TEXT("nan(ind)!"))));
		TestTrue(TEXT("(float conversion from string) infinity"), LexTryParseString(Value, (TEXT("infinity"))));

		// Some numbers with whitespace
		TestTrue(TEXT("(float conversion from string) whitespace"), LexTryParseString(Value, (TEXT("   2.5   "))) && Value == 2.5);
		TestTrue(TEXT("(float conversion from string) whitespace"), LexTryParseString(Value, (TEXT("\t3.0\t"))) && Value == 3.0);
		TestTrue(TEXT("(float conversion from string) whitespace"), LexTryParseString(Value, (TEXT("4.0   \t"))) && Value == 4.0);
		TestTrue(TEXT("(float conversion from string) whitespace"), LexTryParseString(Value, (TEXT("\r\n5.25"))) && Value == 5.25);
		TestTrue(TEXT("(float conversion from string) whitespace"), LexTryParseString(Value, (TEXT(" 6 . 2 "))) && Value == 6.0);
		TestTrue(TEXT("(float conversion from string) whitespace"), LexTryParseString(Value, (TEXT(" 56 . 2 "))) && Value == 56.0);
		TestTrue(TEXT("(float conversion from string) whitespace"), LexTryParseString(Value, (TEXT(" 5 6 . 2 "))) && Value == 5.0);

		// Failure cases
		TestFalse(TEXT("(float no conversion from string) not a number"), LexTryParseString(Value, (TEXT("not a number"))));
		TestFalse(TEXT("(float no conversion from string) <empty string>"), LexTryParseString(Value, (TEXT(""))));
		TestFalse(TEXT("(float conversion from string) ."), LexTryParseString(Value, (TEXT("."))));
	}

	{
		// Test integer values

		int32 Value;

		// Basic numbers
		TestTrue(TEXT("(int32 conversion from string) basic numbers"), LexTryParseString(Value, (TEXT("1"))) && Value == 1);
		TestTrue(TEXT("(int32 conversion from string) basic numbers"), LexTryParseString(Value, (TEXT("1.0"))) && Value == 1);
		TestTrue(TEXT("(int32 conversion from string) basic numbers"), LexTryParseString(Value, (TEXT("3.1"))) && Value == 3);
		TestTrue(TEXT("(int32 conversion from string) basic numbers"), LexTryParseString(Value, (TEXT("0.5"))) && Value == 0);
		TestTrue(TEXT("(int32 conversion from string) basic numbers"), LexTryParseString(Value, (TEXT("1."))) && Value == 1);

		// Variations of 0
		TestTrue(TEXT("(int32 conversion from string) variations of 0"), LexTryParseString(Value, (TEXT("0"))) && Value == 0);
		TestTrue(TEXT("(int32 conversion from string) variations of 0"), LexTryParseString(Value, (TEXT("0.0"))) && Value == 0);
		TestFalse(TEXT("(int32 conversion from string) variations of 0"), LexTryParseString(Value, (TEXT(".0"))) && Value == 0);
		TestTrue(TEXT("(int32 conversion from string) variations of 0"), LexTryParseString(Value, (TEXT("0."))) && Value == 0);

		// Scientific notation
		TestTrue(TEXT("(int32 conversion from string) scientific notation"), LexTryParseString(Value, (TEXT("1.0e+10"))) && Value == 1);
		TestTrue(TEXT("(int32 conversion from string) scientific notation"), LexTryParseString(Value, (TEXT("6.0e-10"))) && Value == 6);
		TestTrue(TEXT("(int32 conversion from string) scientific notation"), LexTryParseString(Value, (TEXT("0.0e+10"))) && Value == 0);
		TestTrue(TEXT("(int32 conversion from string) scientific notation"), LexTryParseString(Value, (TEXT("0.0e-10"))) && Value == 0);
		TestTrue(TEXT("(int32 conversion from string) scientific notation"), LexTryParseString(Value, (TEXT("3e+10"))) && Value == 3);
		TestTrue(TEXT("(int32 conversion from string) scientific notation"), LexTryParseString(Value, (TEXT("4e-10"))) && Value == 4);

		// Some numbers with whitespace
		TestTrue(TEXT("(int32 conversion from string) whitespace"), LexTryParseString(Value, (TEXT("   2.5   "))) && Value == 2);
		TestTrue(TEXT("(int32 conversion from string) whitespace"), LexTryParseString(Value, (TEXT("\t3.0\t"))) && Value == 3);
		TestTrue(TEXT("(int32 conversion from string) whitespace"), LexTryParseString(Value, (TEXT("4.0   \t"))) && Value == 4);
		TestTrue(TEXT("(int32 conversion from string) whitespace"), LexTryParseString(Value, (TEXT("\r\n5.25"))) && Value == 5);
		TestTrue(TEXT("(int32 conversion from string) whitespace"), LexTryParseString(Value, (TEXT(" 6 . 2 "))) && Value == 6);
		TestTrue(TEXT("(int32 conversion from string) whitespace"), LexTryParseString(Value, (TEXT(" 56 . 2 "))) && Value == 56);
		TestTrue(TEXT("(int32 conversion from string) whitespace"), LexTryParseString(Value, (TEXT(" 5 6 . 2 "))) && Value == 5);

		// Non-finite special numbers. All shouldn't parse into an int
		TestFalse(TEXT("(int32 no conversion from string) inf"), LexTryParseString(Value, (TEXT("inf"))));
		TestFalse(TEXT("(int32 no conversion from string) nan"), LexTryParseString(Value, (TEXT("nan"))));
		TestFalse(TEXT("(int32 no conversion from string) nan(ind)"), LexTryParseString(Value, (TEXT("nan(ind)"))));
		TestFalse(TEXT("(int32 no conversion from string) nananananananana"), LexTryParseString(Value, (TEXT("nananananananana"))));
		TestFalse(TEXT("(int32 no conversion from string) nan(ind)!"), LexTryParseString(Value, (TEXT("nan(ind)!"))));
		TestFalse(TEXT("(int32 no conversion from string) infinity"), LexTryParseString(Value, (TEXT("infinity"))));
		TestFalse(TEXT("(float no conversion from string) ."), LexTryParseString(Value, (TEXT("."))));
		TestFalse(TEXT("(float no conversion from string) <empyty string>"), LexTryParseString(Value, (TEXT(""))));
	}

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FStringParseIntoArrayTest, "System.Core.String.ParseIntoArray", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)

bool FStringParseIntoArrayTest::RunTest(const FString& Parameters)
{
	FString Empty;
	FString EmptyButAllocated = "This|is|a|long|string";
	EmptyButAllocated.GetCharArray().Reset();

	const TCHAR* Delim[] = { TEXT("|") };

	// Test that an array of 1 delimeter and single delimeters work the same way
	{
		TArray<FString> Parsed;

		Parsed.Reset(); FString("|")     .ParseIntoArray(Parsed, Delim[0], /*cull = */true);  TestTrue(TEXT("parse a single delimiter character with a single delimiter, culling empties"),            Parsed == TArray<FString>{});
		Parsed.Reset(); FString("|")     .ParseIntoArray(Parsed, Delim, 1, /*cull = */true);  TestTrue(TEXT("parse a single delimiter character with an array of 1 delimiter, culling empties"),       Parsed == TArray<FString>{});
		Parsed.Reset(); FString{}        .ParseIntoArray(Parsed, Delim[0], /*cull = */true);  TestTrue(TEXT("parse an unallocated string with a single delimiter, culling empties"),                   Parsed == TArray<FString>{});
		Parsed.Reset(); FString{}        .ParseIntoArray(Parsed, Delim, 1, /*cull = */true);  TestTrue(TEXT("parse an unallocated string with an array of 1 delimiter, culling empties"),              Parsed == TArray<FString>{});
		Parsed.Reset(); EmptyButAllocated.ParseIntoArray(Parsed, Delim[0], /*cull = */true);  TestTrue(TEXT("parse an allocated empty string with a single delimiter, culling empties"),               Parsed == TArray<FString>{});
		Parsed.Reset(); EmptyButAllocated.ParseIntoArray(Parsed, Delim, 1, /*cull = */true);  TestTrue(TEXT("parse an allocated empty string with an array of 1 delimiter, culling empties"),          Parsed == TArray<FString>{});
		Parsed.Reset(); FString("a|")    .ParseIntoArray(Parsed, Delim[0], /*cull = */true);  TestTrue(TEXT("parse a string with a trailing delimiter with a single delimiter, culling empties"),      Parsed == TArray<FString>{TEXT("a")});
		Parsed.Reset(); FString("a|")    .ParseIntoArray(Parsed, Delim, 1, /*cull = */true);  TestTrue(TEXT("parse a string with a trailing delimiter with an array of 1 delimiter, culling empties"), Parsed == TArray<FString>{TEXT("a")});
		Parsed.Reset(); FString("|b")    .ParseIntoArray(Parsed, Delim[0], /*cull = */true);  TestTrue(TEXT("parse a string with a leading delimiter with a single delimiter, culling empties"),       Parsed == TArray<FString>{TEXT("b")});
		Parsed.Reset(); FString("|b")    .ParseIntoArray(Parsed, Delim, 1, /*cull = */true);  TestTrue(TEXT("parse a string with a leading delimiter with an array of 1 delimiter, culling empties"),  Parsed == TArray<FString>{TEXT("b")});
		Parsed.Reset(); FString("a|b")   .ParseIntoArray(Parsed, Delim[0], /*cull = */true);  TestTrue(TEXT("parse a string with a middle delimiter with a single delimiter, culling empties"),        Parsed == TArray<FString>{TEXT("a"), TEXT("b")});
		Parsed.Reset(); FString("a|b")   .ParseIntoArray(Parsed, Delim, 1, /*cull = */true);  TestTrue(TEXT("parse a string with a middle delimiter with an array of 1 delimiter, culling empties"),   Parsed == TArray<FString>{TEXT("a"), TEXT("b")});
		Parsed.Reset(); FString("|")     .ParseIntoArray(Parsed, Delim[0], /*cull = */false); TestTrue(TEXT("parse a single delimiter character with a single delimiter, keeping empties"),            Parsed == TArray<FString>{TEXT(""), TEXT("")});
		Parsed.Reset(); FString("|")     .ParseIntoArray(Parsed, Delim, 1, /*cull = */false); TestTrue(TEXT("parse a single delimiter character with an array of 1 delimiter, keeping empties"),       Parsed == TArray<FString>{TEXT(""), TEXT("")});
		Parsed.Reset(); FString{}        .ParseIntoArray(Parsed, Delim[0], /*cull = */false); TestTrue(TEXT("parse an unallocated string with a single delimiter, keeping empties"),                   Parsed == TArray<FString>{TEXT("")});
		Parsed.Reset(); FString{}        .ParseIntoArray(Parsed, Delim, 1, /*cull = */false); TestTrue(TEXT("parse an unallocated string with an array of 1 delimiter, keeping empties"),              Parsed == TArray<FString>{TEXT("")});
		Parsed.Reset(); EmptyButAllocated.ParseIntoArray(Parsed, Delim[0], /*cull = */false); TestTrue(TEXT("parse an allocated empty string with a single delimiter, keeping empties"),               Parsed == TArray<FString>{TEXT("")});
		Parsed.Reset(); EmptyButAllocated.ParseIntoArray(Parsed, Delim, 1, /*cull = */false); TestTrue(TEXT("parse an allocated empty string with an array of 1 delimiter, keeping empties"),          Parsed == TArray<FString>{TEXT("")});
		Parsed.Reset(); FString("a|")    .ParseIntoArray(Parsed, Delim[0], /*cull = */false); TestTrue(TEXT("parse a string with a trailing delimiter with a single delimiter, keeping empties"),      Parsed == TArray<FString>{TEXT("a"), TEXT("")});
		Parsed.Reset(); FString("a|")    .ParseIntoArray(Parsed, Delim, 1, /*cull = */false); TestTrue(TEXT("parse a string with a trailing delimiter with an array of 1 delimiter, keeping empties"), Parsed == TArray<FString>{TEXT("a"), TEXT("")});
		Parsed.Reset(); FString("|b")    .ParseIntoArray(Parsed, Delim[0], /*cull = */false); TestTrue(TEXT("parse a string with a leading delimiter with a single delimiter, keeping empties"),       Parsed == TArray<FString>{TEXT(""), TEXT("b")});
		Parsed.Reset(); FString("|b")    .ParseIntoArray(Parsed, Delim, 1, /*cull = */false); TestTrue(TEXT("parse a string with a leading delimiter with an array of 1 delimiter, keeping empties"),  Parsed == TArray<FString>{TEXT(""), TEXT("b")});
		Parsed.Reset(); FString("a|b")   .ParseIntoArray(Parsed, Delim[0], /*cull = */false); TestTrue(TEXT("parse a string with a middle delimiter with a single delimiter, keeping empties"),        Parsed == TArray<FString>{TEXT("a"), TEXT("b")});
		Parsed.Reset(); FString("a|b")   .ParseIntoArray(Parsed, Delim, 1, /*cull = */false); TestTrue(TEXT("parse a string with a middle delimiter with an array of 1 delimiter, keeping empties"),   Parsed == TArray<FString>{TEXT("a"), TEXT("b")});
	}

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FStringSubstringTest, "System.Core.String.Substring", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool FStringSubstringTest::RunTest(const FString& Parameters)
{
	auto DoTest = [this](const TCHAR* Call, const FString& Result, const TCHAR* InExpected)
	{
		if (!Result.Equals(InExpected, ESearchCase::CaseSensitive))
		{
			AddError(FString::Printf(TEXT("'%s' failure: result '%s' (expected '%s')"), Call, *Result, InExpected));
		}
	};

	const FString TestString("0123456789");

#define SUBSTRINGTEST(TestName, ExpectedResult, Operation, ...) \
	FString TestName = TestString.Operation(__VA_ARGS__); \
	DoTest(TEXT(#TestName), TestName, ExpectedResult); \
	\
	FString Inline##TestName = TestString; \
	Inline##TestName.Operation##Inline(__VA_ARGS__); \
	DoTest(TEXT("Inline" #TestName), Inline##TestName, ExpectedResult); 

	// Left
	SUBSTRINGTEST(Left, TEXT("0123"), Left, 4);
	SUBSTRINGTEST(ExactLengthLeft, *TestString, Left, 10);
	SUBSTRINGTEST(LongerThanLeft, *TestString, Left, 20);
	SUBSTRINGTEST(ZeroLeft, TEXT(""), Left, 0);
	SUBSTRINGTEST(NegativeLeft, TEXT(""), Left, -1);

	// LeftChop
	SUBSTRINGTEST(LeftChop, TEXT("012345"), LeftChop, 4);
	SUBSTRINGTEST(ExactLengthLeftChop, TEXT(""), LeftChop, 10);
	SUBSTRINGTEST(LongerThanLeftChop, TEXT(""), LeftChop, 20);
	SUBSTRINGTEST(ZeroLeftChop, *TestString, LeftChop, 0);
	SUBSTRINGTEST(NegativeLeftChop, *TestString, LeftChop, -1);

	// Right
	SUBSTRINGTEST(Right, TEXT("6789"), Right, 4);
	SUBSTRINGTEST(ExactLengthRight, *TestString, Right, 10);
	SUBSTRINGTEST(LongerThanRight, *TestString, Right, 20);
	SUBSTRINGTEST(ZeroRight, TEXT(""), Right, 0);
	SUBSTRINGTEST(NegativeRight, TEXT(""), Right, -1);

	// RightChop
	SUBSTRINGTEST(RightChop, TEXT("456789"), RightChop, 4);
	SUBSTRINGTEST(ExactLengthRightChop, TEXT(""), RightChop, 10);
	SUBSTRINGTEST(LongerThanRightChop, TEXT(""), RightChop, 20);
	SUBSTRINGTEST(ZeroRightChop, *TestString, RightChop, 0);
	SUBSTRINGTEST(NegativeRightChop, *TestString, RightChop, -1);

	// Mid
	SUBSTRINGTEST(Mid, TEXT("456789"), Mid, 4);
	SUBSTRINGTEST(MidCount, TEXT("4567"), Mid, 4, 4);
	SUBSTRINGTEST(MidCountFullLength, *TestString, Mid, 0, 10);
	SUBSTRINGTEST(MidCountOffEnd, TEXT("89"), Mid, 8, 4);
	SUBSTRINGTEST(MidStartAfterEnd, TEXT(""), Mid, 20);
	SUBSTRINGTEST(MidZeroCount, TEXT(""), Mid, 5, 0);
	SUBSTRINGTEST(MidNegativeCount, TEXT(""), Mid, 5, -1);
	SUBSTRINGTEST(MidNegativeStartNegativeEnd, TEXT(""), Mid, -5, 1);
	SUBSTRINGTEST(MidNegativeStartPositiveEnd, TEXT("012"), Mid, -1, 4);
	SUBSTRINGTEST(MidNegativeStartBeyondEnd, *TestString, Mid, -1, 15);

#undef SUBSTRINGTEST

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FStringFromStringViewTest, "System.Core.String.FromStringView", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool FStringFromStringViewTest::RunTest(const FString& Parameters)
{
	// Verify basic construction and assignment from a string view.
	{
		const TCHAR* Literal = TEXT("Literal");
		const ANSICHAR* AnsiLiteral = "Literal";
		TestEqual(TEXT("String(StringView)"), FString(FStringView(Literal)), Literal);
		TestEqual(TEXT("String(AnsiStringView)"), FString(FAnsiStringView(AnsiLiteral)), Literal);
		TestEqual(TEXT("String = StringView"), FString("Temp") = FStringView(Literal), Literal);

		FStringView EmptyStringView;
		FString EmptyString(EmptyStringView);
		TestTrue(TEXT("String(EmptyStringView)"), EmptyString.IsEmpty());
		TestTrue(TEXT("String(EmptyStringView) (No Allocation)"), EmptyString.GetAllocatedSize() == 0);

		EmptyString = TEXT("Temp");
		EmptyString = EmptyStringView;
		TestTrue(TEXT("String = EmptyStringView"), EmptyString.IsEmpty());
		TestTrue(TEXT("String = EmptyStringView (No Allocation)"), EmptyString.GetAllocatedSize() == 0);
	}

	// Verify assignment from a view of itself.
	{
		FString AssignEntireString("AssignEntireString");
		AssignEntireString = FStringView(AssignEntireString);
		TestEqual(TEXT("String = StringView(String)"), AssignEntireString, TEXT("AssignEntireString"));

		FString AssignStartOfString("AssignStartOfString");
		AssignStartOfString = FStringView(AssignStartOfString).Left(11);
		TestEqual(TEXT("String = StringView(String).Left"), AssignStartOfString, TEXT("AssignStart"));

		FString AssignEndOfString("AssignEndOfString");
		AssignEndOfString = FStringView(AssignEndOfString).Right(11);
		TestEqual(TEXT("String = StringView(String).Right"), AssignEndOfString, TEXT("EndOfString"));

		FString AssignMiddleOfString("AssignMiddleOfString");
		AssignMiddleOfString = FStringView(AssignMiddleOfString).Mid(6, 6);
		TestEqual(TEXT("String = StringView(String).Mid"), AssignMiddleOfString, TEXT("Middle"));
	}

	// Verify operators taking string views and character arrays
	{
		FStringView RhsStringView = FStringView(TEXT("RhsNotSZ"), 3);
		FString MovePlusSVResult = FString("Lhs") + RhsStringView;
		TestEqual(TEXT("Move String + StringView"), MovePlusSVResult, TEXT("LhsRhs"));

		FString CopyLhs("Lhs");
		FString CopyPlusSVResult = CopyLhs + RhsStringView;
		TestEqual(TEXT("Copy String + StringView"), CopyPlusSVResult, TEXT("LhsRhs"));

		FString MovePlusTCHARsResult = FString("Lhs") + TEXT("Rhs");
		TestEqual(TEXT("Move String + TCHAR*"), MovePlusTCHARsResult, TEXT("LhsRhs"));

		FString CopyPlusTCHARsResult = CopyLhs + TEXT("Rhs");
		TestEqual(TEXT("Copy String + TCHAR*"), CopyPlusTCHARsResult, TEXT("LhsRhs"));

		FStringView LhsStringView = FStringView(TEXT("LhsNotSZ"), 3);
		FString SVPlusMoveResult = LhsStringView + FString("Rhs");
		TestEqual(TEXT("StringView + Move String"), SVPlusMoveResult, TEXT("LhsRhs"));

		FString CopyRhs("Rhs");
		FString SVPlusCopyResult = LhsStringView + CopyRhs;
		TestEqual(TEXT("StringView + Copy String"), SVPlusCopyResult, TEXT("LhsRhs"));

		FString TCHARsPlusMoveResult = TEXT("Lhs") + FString("Rhs");
		TestEqual(TEXT("TCHAR* + Move String"), TCHARsPlusMoveResult, TEXT("LhsRhs"));

		FString TCHARsPlusCopyResult = TEXT("Lhs") + CopyRhs;
		TestEqual(TEXT("TCHAR* + Copy String"), TCHARsPlusCopyResult, TEXT("LhsRhs"));
	}

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FStringConstructorWithSlackTest, "System.Core.String.ConstructorWithSlack", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool FStringConstructorWithSlackTest::RunTest(const FString& Parameters)
{
	// Note that the total capacity of a string might be greater than the string length + slack + a null terminator due to
	// underlying malloc implementations which is why we poll FMemory to see what size of allocation we should be expecting.

	// Test creating from a valid string with various valid slack value
	{
		const TCHAR* TestString = TEXT("FooBar");
		const char* TestAsciiString = "FooBar";
		const int32 LengthOfString = TCString<TCHAR>::Strlen(TestString);
		const int32 ExtraSlack = 32;
		const int32 NumElements = LengthOfString + ExtraSlack + 1;

		const SIZE_T ExpectedCapacity = FMemory::QuantizeSize(NumElements * sizeof(TCHAR));

		FString StringFromTChar = FString::ConstructWithSlack(TestString, ExtraSlack);
		TestEqual(TEXT("(TCHAR: Valid string with valid slack) resulting capacity"), StringFromTChar.GetAllocatedSize(), ExpectedCapacity);

		FString StringFromAscii = FString::ConstructWithSlack(TestAsciiString, ExtraSlack);
		TestEqual(TEXT("(ASCII: Valid string with valid slack) resulting capacity"), StringFromAscii.GetAllocatedSize(), ExpectedCapacity);

		FString StringFromFStringView(FStringView(TestString), ExtraSlack);
		TestEqual(TEXT("(FStringView: Valid string with valid slack) resulting capacity"), StringFromFStringView.GetAllocatedSize(), ExpectedCapacity);

		FString StringFromFString(FString(TestString), ExtraSlack);
		TestEqual(TEXT("(FString: Valid string with valid slack"), StringFromFString.GetAllocatedSize(), ExpectedCapacity);
	}

	// Test creating from a valid string with a zero slack value
	{
		const TCHAR* TestString = TEXT("FooBar");
		const char* TestAsciiString = "FooBar";
		const int32 LengthOfString = TCString<TCHAR>::Strlen(TestString);
		const int32 ExtraSlack = 0;
		const int32 NumElements = LengthOfString + ExtraSlack + 1;

		const SIZE_T ExpectedCapacity = FMemory::QuantizeSize(NumElements * sizeof(TCHAR));

		FString StringFromTChar = FString::ConstructWithSlack(TestString, ExtraSlack);
		TestEqual(TEXT("(TCHAR: Valid string with zero slack) resulting capacity"), StringFromTChar.GetAllocatedSize(), ExpectedCapacity);

		FString StringFromAscii = FString::ConstructWithSlack(TestAsciiString, ExtraSlack);
		TestEqual(TEXT("(ASCII: Valid string with zero slack) resulting capacity"), StringFromAscii.GetAllocatedSize(), ExpectedCapacity);

		FString StringFromFStringView(FStringView(TestString), ExtraSlack);
		TestEqual(TEXT("(FStringView: Valid string with zero slack) resulting capacity"), StringFromFStringView.GetAllocatedSize(), ExpectedCapacity);

		FString StringFromFString(FString(TestString), ExtraSlack);
		TestEqual(TEXT("(FString: Valid string with zero slack"), StringFromFString.GetAllocatedSize(), ExpectedCapacity);
	}

	// Test creating from an empty string with a valid slack value
	{
		const TCHAR* TestString = TEXT("");
		const char* TestAsciiString = "";
		const int32 LengthOfString = TCString<TCHAR>::Strlen(TestString);
		const int32 ExtraSlack = 32;
		const int32 NumElements = LengthOfString + ExtraSlack + 1;

		const SIZE_T ExpectedCapacity = FMemory::QuantizeSize(NumElements * sizeof(TCHAR));

		FString StringFromTChar = FString::ConstructWithSlack(TestString, ExtraSlack);
		TestEqual(TEXT("(TCHAR: Empty string with slack) resulting capacity"), StringFromTChar.GetAllocatedSize(), ExpectedCapacity);

		FString StringFromAscii = FString::ConstructWithSlack(TestAsciiString, ExtraSlack);
		TestEqual(TEXT("(ASCII: Empty string with slack) resulting capacity"), StringFromAscii.GetAllocatedSize(), ExpectedCapacity);

		FString StringFromFStringView(FStringView(TestString), ExtraSlack);
		TestEqual(TEXT("(FStringView: Empty string with slack) resulting capacity"), StringFromFStringView.GetAllocatedSize(), ExpectedCapacity);

		FString StringFromFString(FString(TestString), ExtraSlack);
		TestEqual(TEXT("(FString: Empty string with slack) resulting capacity"), StringFromFString.GetAllocatedSize(), ExpectedCapacity);
	}

	// Test creating from an empty string with a zero slack value
	{
		const TCHAR* TestString = TEXT("");
		const char* TestAsciiString = "";
		const int32 ExtraSlack = 0;

		const SIZE_T ExpectedCapacity = 0u;

		FString StringFromTChar = FString::ConstructWithSlack(TestString, ExtraSlack);
		TestEqual(TEXT("(TCHAR: Empty string with zero slack) resulting capacity"), StringFromTChar.GetAllocatedSize(), ExpectedCapacity);

		FString StringFromAscii = FString::ConstructWithSlack(TestAsciiString, ExtraSlack);
		TestEqual(TEXT("(ASCII: Empty string with zero slack) resulting capacity"), StringFromAscii.GetAllocatedSize(), ExpectedCapacity);

		FString StringFromFStringView(FStringView(TestString), ExtraSlack);
		TestEqual(TEXT("(FStringView: Empty string with zero slack) resulting capacity"), StringFromFStringView.GetAllocatedSize(), ExpectedCapacity);

		FString StringFromFString(FString(TestString), ExtraSlack);
		TestEqual(TEXT("(FString: Empty string with zero slack) resulting capacity"), StringFromFString.GetAllocatedSize(), ExpectedCapacity);
	}

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FStringEqualityTest, "System.Core.String.Equality", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool FStringEqualityTest::RunTest(const FString& Parameters)
{
	auto TestSelfEquality = [this](const TCHAR* A)
	{
		TestTrue(TEXT("Self Equality C string"), FString(A) == A);
		TestTrue(TEXT("Self Equality C string"), A == FString(A));
		TestTrue(TEXT("Self Equality CaseSensitive"), FString(A).Equals(FString(A), ESearchCase::CaseSensitive));
		TestTrue(TEXT("Self Equality IgnoreCase"), FString(A).Equals(FString(A), ESearchCase::IgnoreCase));

		FString Slacker(A);
		Slacker.Reserve(100);
		TestTrue(TEXT("Self Equality slack"), Slacker == FString(A));
	};

	auto TestPairEquality = [this](const TCHAR* A, const TCHAR* B)
	{
		TestEqual(TEXT("Equals CaseSensitive"), FCString::Strcmp(A, B)  == 0, FString(A).Equals(FString(B), ESearchCase::CaseSensitive));
		TestEqual(TEXT("Equals CaseSensitive"), FCString::Strcmp(B, A)  == 0, FString(B).Equals(FString(A), ESearchCase::CaseSensitive));
		TestEqual(TEXT("Equals IgnoreCase"),	FCString::Stricmp(A, B) == 0, FString(A).Equals(FString(B), ESearchCase::IgnoreCase));
		TestEqual(TEXT("Equals IgnoreCase"),	FCString::Stricmp(B, A) == 0, FString(B).Equals(FString(A), ESearchCase::IgnoreCase));
	};

	const TCHAR* Pairs[][2] =	{ {TEXT(""),	TEXT(" ")}
								, {TEXT("a"),	TEXT("A")}
								, {TEXT("aa"),	TEXT("aA")}
								, {TEXT("az"),	TEXT("AZ")}
								, {TEXT("@["),	TEXT("@]")} };

	for (const TCHAR** Pair : Pairs)
	{
		TestSelfEquality(Pair[0]);
		TestSelfEquality(Pair[1]);
		TestPairEquality(Pair[0], Pair[1]);
	}

	return true;	
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FStringPathConcatCompoundOperatorTest, "System.Core.String.PathConcatCompoundOperator", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool FStringPathConcatCompoundOperatorTest::RunTest(const FString& Parameters)
{
	// No need to test a nullptr TCHAR* as an input parameter as this is expected to cause a crash
	// No need to test self assignment, clang will catch that was a compiler error (-Wself-assign-overloaded)

	const TCHAR* Path = TEXT("../Path");
	const TCHAR* PathWithTrailingSlash = TEXT("../Path/");
	const TCHAR* Filename = TEXT("File.txt");
	const TCHAR* FilenameWithLeadingSlash = TEXT("/File.txt");
	const TCHAR* CombinedPath = TEXT("../Path/File.txt");
	const TCHAR* CombinedPathWithDoubleSeparator = TEXT("../Path//File.txt");

	// Existing code supported ansi char so we need to test that to avoid potentially breaking license code
	const ANSICHAR* AnsiFilename = "File.txt";
	const ANSICHAR* AnsiFilenameWithLeadingSlash = "/File.txt";

	// The TStringBuilders must be created up front as no easy constructor
	TStringBuilder<128> EmptyStringBuilder;
	TStringBuilder<128> FilenameStringBuilder; FilenameStringBuilder << Filename;
	TStringBuilder<128> FilenameWithLeadingSlashStringBuilder; FilenameWithLeadingSlashStringBuilder << FilenameWithLeadingSlash;
	
#define TEST_EMPTYPATH_EMPTYFILE(Type, Input)						{ FString EmptyPathString; EmptyPathString /= Input; TestTrue(TEXT(Type ": EmptyPath/EmptyFilename to be empty"), EmptyPathString.IsEmpty()); }
#define TEST_VALIDPATH_EMPTYFILE(Type, Input)						{ FString Result(Path); Result /= Input; TestEqual(TEXT(Type ": ValidPath/EmptyFilename result to be"), Result, PathWithTrailingSlash); } \
																	{ FString Result(PathWithTrailingSlash); Result /= Input; TestEqual(TEXT(Type " (with extra /): ValidPath/EmptyFilename result to be"), Result, PathWithTrailingSlash); }	
#define TEST_EMPTYPATH_VALIDFILE(Type, Input)						{ FString Result; Result /= Input; TestEqual(TEXT(Type ": EmptyPath/ValidFilename"), Result, Filename); }
#define TEST_VALIDPATH_VALIDFILE(Type, Path, File)					{ FString Result(Path); Result /= File; TestEqual(TEXT(Type ": ValidPath/ValidFilename"), Result, CombinedPath); }
#define TEST_VALIDPATH_VALIDFILE_DOUBLE_SEPARATOR(Type, Path, File)	{ FString Result(Path); Result /= File; TestEqual(TEXT(Type ": ValidPath//ValidFilename"), Result, CombinedPathWithDoubleSeparator); }

	// Test empty path /= empty file
	TEST_EMPTYPATH_EMPTYFILE("NullString", FString());
	TEST_EMPTYPATH_EMPTYFILE("EmptyString", FString());
	TEST_EMPTYPATH_EMPTYFILE("EmptyAnsiLiteralString", "");
	TEST_EMPTYPATH_EMPTYFILE("EmptyLiteralString", TEXT(""));
	TEST_EMPTYPATH_EMPTYFILE("NullStringView", FStringView());
	TEST_EMPTYPATH_EMPTYFILE("EmptyStringView", FStringView(TEXT("")));
	TEST_EMPTYPATH_EMPTYFILE("EmptyStringBuilder", EmptyStringBuilder);

	// Test valid path /= empty file
	TEST_VALIDPATH_EMPTYFILE("NullString", FString());
	TEST_VALIDPATH_EMPTYFILE("EmptyString", FString());
	TEST_VALIDPATH_EMPTYFILE("EmptyAnsiLiteralString", "");
	TEST_VALIDPATH_EMPTYFILE("EmptyLiteralString", TEXT(""));
	TEST_VALIDPATH_EMPTYFILE("NullStringView", FStringView());
	TEST_VALIDPATH_EMPTYFILE("EmptyStringView", FStringView(TEXT("")));
	TEST_VALIDPATH_EMPTYFILE("EmptyStringBuilder", EmptyStringBuilder);
	
	// Test empty path /= valid file
	TEST_EMPTYPATH_VALIDFILE("String", FString(Filename));
	TEST_EMPTYPATH_VALIDFILE("LiteralString", Filename);
	TEST_EMPTYPATH_VALIDFILE("LiteralAnsiString", AnsiFilename);
	TEST_EMPTYPATH_VALIDFILE("StringView", FStringView(Filename));
	
	// Test valid path /= valid file
	TEST_VALIDPATH_VALIDFILE("String", Path, FString(Filename));
	TEST_VALIDPATH_VALIDFILE("LiteralString", Path, Filename);
	TEST_VALIDPATH_VALIDFILE("LiteralAnsiString", Path, AnsiFilename);
	TEST_VALIDPATH_VALIDFILE("StringView", Path, FStringView(Filename));
	TEST_VALIDPATH_VALIDFILE("StringBuilder", Path, FilenameStringBuilder);

	// Test valid path (ending in /) /= valid file
	TEST_VALIDPATH_VALIDFILE("String (path with extra /)", PathWithTrailingSlash, FString(Filename));
	TEST_VALIDPATH_VALIDFILE("LiteralString (path with extra /)", PathWithTrailingSlash, Filename);
	TEST_VALIDPATH_VALIDFILE("LiteralAnsiString (path with extra /)", PathWithTrailingSlash, AnsiFilename);
	TEST_VALIDPATH_VALIDFILE("StringView (path with extra /)", PathWithTrailingSlash, FStringView(Filename));
	TEST_VALIDPATH_VALIDFILE("StringBuilder (path with extra /)", PathWithTrailingSlash, FilenameStringBuilder);
	
	// Test valid path / valid path + file (starting with /)
	TEST_VALIDPATH_VALIDFILE("String (filename with extra /)", Path, FString(FilenameWithLeadingSlash));
	TEST_VALIDPATH_VALIDFILE("LiteralString (filename with extra /)", Path, FilenameWithLeadingSlash);
	TEST_VALIDPATH_VALIDFILE("LiteralAnsiString (filename with extra /)", Path, AnsiFilenameWithLeadingSlash);
	TEST_VALIDPATH_VALIDFILE("StringView (filename with extra /)", Path, FStringView(FilenameWithLeadingSlash));
	TEST_VALIDPATH_VALIDFILE("StringBuilder (filename with extra /)", Path, FilenameWithLeadingSlashStringBuilder);
	
	// Appending a file name that starts with a / to a directory that ends with a / will not remove the erroneous / and so 
	// will end up with // in the path, these tests are to show this behavior
	// For example "path/" /= "/file.txt" will result in "path//file.txt" not "path/file.txt"
	TEST_VALIDPATH_VALIDFILE_DOUBLE_SEPARATOR("String (path and filename with extra /)", PathWithTrailingSlash, FString(FilenameWithLeadingSlash));
	TEST_VALIDPATH_VALIDFILE_DOUBLE_SEPARATOR("LiteralString (path and filename with extra /)", PathWithTrailingSlash, FilenameWithLeadingSlash);
	TEST_VALIDPATH_VALIDFILE_DOUBLE_SEPARATOR("LiteralAnsiString (path and filename with extra /)", PathWithTrailingSlash, AnsiFilenameWithLeadingSlash);
	TEST_VALIDPATH_VALIDFILE_DOUBLE_SEPARATOR("StringView (path and filename with extra /)", PathWithTrailingSlash, FStringView(FilenameWithLeadingSlash));
	TEST_VALIDPATH_VALIDFILE_DOUBLE_SEPARATOR("StringBuilder (path and filename with extra /)", PathWithTrailingSlash, FilenameWithLeadingSlashStringBuilder);

#undef TEST_EMPTYPATH_EMPTYFILE
#undef TEST_VALIDPATH_EMPTYFILE
#undef TEST_VALIDPATH_EMPTYFILE
#undef TEST_VALIDPATH_VALIDFILE

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FStringFindAndContainsTest, "System.Core.String.FindAndContains", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
bool FStringFindAndContainsTest::RunTest(const FString& Parameters)
{
	auto RunTest = [this](FStringView Search, FStringView Find, ESearchCase::Type SearchCase,
		ESearchDir::Type SearchDir, int32 StartPosition, int32 Expected)
	{
		FString SearchStr(Search);
		FString FindStr(Find);
		bool bRunContainsTest = (SearchDir == ESearchDir::FromStart && StartPosition == 0) ||
			(SearchDir == ESearchDir::FromEnd && StartPosition >= SearchStr.Len());
		// TCHAR*, int32
		{
			int32 Actual = SearchStr.Find(Find.GetData(), Find.Len(), SearchCase, SearchDir, StartPosition);
			if (Actual != Expected)
			{
				AddError(FString::Printf(TEXT("FString(\"%s\").Find(\"%s\", %d, %d, %d, %d) returned Actual %d not equal to Expected %d"),
					*SearchStr, *FindStr, FindStr.Len(), (int32)SearchCase, (int32)SearchDir, StartPosition, Actual, Expected));
			}
			if (bRunContainsTest)
			{
				bool ContainsExpected = Expected != INDEX_NONE;
				bool ContainsActual = SearchStr.Contains(Find.GetData(), Find.Len(), SearchCase, SearchDir);
				if (ContainsActual != ContainsExpected)
				{
					AddError(FString::Printf(TEXT("FString(\"%s\").Contains(\"%s\", %d, %d, %d) returned Actual %s not equal to Expected %s"),
						*SearchStr, *FindStr, FindStr.Len(), (int32)SearchCase, (int32)SearchDir,  *LexToString(ContainsActual), *LexToString(ContainsExpected)));
				}
			}
		}
		// FStringView
		{
			int32 Actual = SearchStr.Find(Find, SearchCase, SearchDir, StartPosition);
			if (Actual != Expected)
			{
				AddError(FString::Printf(TEXT("FString(\"%s\").Find(FStringView(\"%s\", %d), %d, %d, %d) returned Actual %d not equal to Expected %d"),
					*SearchStr, *FindStr, FindStr.Len(), (int32)SearchCase, (int32)SearchDir, StartPosition, Actual, Expected));
			}
			if (bRunContainsTest)
			{
				bool ContainsExpected = Expected != INDEX_NONE;
				bool ContainsActual = ContainsExpected;
				if (ContainsActual != ContainsExpected)
				{
					AddError(FString::Printf(TEXT("FString(\"%s\").Contains(FStringView(\"%s\", %d), %d, %d) returned Actual %s not equal to Expected %s"),
						*SearchStr, *FindStr, FindStr.Len(), (int32)SearchCase, (int32)SearchDir, *LexToString(ContainsActual), *LexToString(ContainsExpected)));
				}
			}
		}

		// TCHAR*, nullterminated
		{
			int32 Actual = SearchStr.Find(*FindStr, SearchCase, SearchDir, StartPosition);
			if (Actual != Expected)
			{
				AddError(FString::Printf(TEXT("FString(\"%s\").Find(TEXT(\"%s\"), %d, %d, %d) returned Actual %d not equal to Expected %d"),
					*SearchStr, *FindStr, (int32)SearchCase, (int32)SearchDir, StartPosition, Actual, Expected));
			}
			if (bRunContainsTest)
			{
				bool ContainsExpected = Expected != INDEX_NONE;
				bool ContainsActual = SearchStr.Contains(*FindStr, SearchCase, SearchDir);
				if (ContainsActual != ContainsExpected)
				{
					AddError(FString::Printf(TEXT("FString(\"%s\").Contains(TEXT(\"%s\"), %d, %d) returned Actual %s not equal to Expected %s"),
						*SearchStr, *FindStr, (int32)SearchCase, (int32)SearchDir, *LexToString(ContainsActual), *LexToString(ContainsExpected)));
				}
			}
		}

		// FString
		{
			int32 Actual = SearchStr.Find(FindStr, SearchCase, SearchDir, StartPosition);
			if (Actual != Expected)
			{
				AddError(FString::Printf(TEXT("FString(\"%s\").Find(FString(%d, \"%s\"), %d, %d, %d) returned Actual %d not equal to Expected %d"),
					*SearchStr, FindStr.Len(), *FindStr, (int32)SearchCase, (int32)SearchDir, StartPosition, Actual, Expected));
			}
			if (bRunContainsTest)
			{
				bool ContainsExpected = Expected != INDEX_NONE;
				bool ContainsActual = SearchStr.Contains(FindStr, SearchCase, SearchDir);
				if (ContainsActual != ContainsExpected)
				{
					AddError(FString::Printf(TEXT("FString(\"%s\").Contains(FString(%d, \"%s\"), %d, %d) returned Actual %s not equal to Expected %s"),
						*SearchStr, FindStr.Len(), *FindStr, (int32)SearchCase, (int32)SearchDir, *LexToString(ContainsActual), *LexToString(ContainsExpected)));
				}
			}
		}
	};
	FStringView ABACADAB(TEXTVIEW("ABACADAB"));
	FStringView A(TEXTVIEW("A"));
	FStringView B(TEXTVIEW("B"));
	FStringView CAD(TEXTVIEW("CAD"));
	FStringView a(TEXTVIEW("a"));
	FStringView EmptyString;
	RunTest(ABACADAB, A, ESearchCase::CaseSensitive, ESearchDir::FromStart, 0, 0);
	RunTest(ABACADAB, A, ESearchCase::CaseSensitive, ESearchDir::FromStart, 1, 2);
	RunTest(ABACADAB, A, ESearchCase::CaseSensitive, ESearchDir::FromStart, 7, INDEX_NONE);
	RunTest(ABACADAB, A, ESearchCase::CaseSensitive, ESearchDir::FromStart, 8, INDEX_NONE);
	RunTest(ABACADAB, A, ESearchCase::CaseSensitive, ESearchDir::FromStart, 80, INDEX_NONE);
	RunTest(ABACADAB, A, ESearchCase::CaseSensitive, ESearchDir::FromEnd, 80, 6);
	RunTest(ABACADAB, A, ESearchCase::CaseSensitive, ESearchDir::FromEnd, 8, 6);
	RunTest(ABACADAB, A, ESearchCase::CaseSensitive, ESearchDir::FromEnd, 7, 6);
	RunTest(ABACADAB, A, ESearchCase::CaseSensitive, ESearchDir::FromEnd, 6, 4);
	RunTest(ABACADAB, A, ESearchCase::CaseSensitive, ESearchDir::FromEnd, 1, 0);
	RunTest(ABACADAB, A, ESearchCase::CaseSensitive, ESearchDir::FromEnd, 0, INDEX_NONE);

	RunTest(ABACADAB, B, ESearchCase::CaseSensitive, ESearchDir::FromStart, 0, 1);
	RunTest(ABACADAB, B, ESearchCase::CaseSensitive, ESearchDir::FromStart, 1, 1);
	RunTest(ABACADAB, B, ESearchCase::CaseSensitive, ESearchDir::FromStart, 2, 7);
	RunTest(ABACADAB, B, ESearchCase::CaseSensitive, ESearchDir::FromStart, 7, 7);
	RunTest(ABACADAB, B, ESearchCase::CaseSensitive, ESearchDir::FromStart, 8, 7); // StartPosition clamped to [0, Len-1]
	RunTest(ABACADAB, B, ESearchCase::CaseSensitive, ESearchDir::FromEnd, 80, 7);
	RunTest(ABACADAB, B, ESearchCase::CaseSensitive, ESearchDir::FromEnd, 8, 7);
	RunTest(ABACADAB, B, ESearchCase::CaseSensitive, ESearchDir::FromEnd, 7, 1);
	RunTest(ABACADAB, B, ESearchCase::CaseSensitive, ESearchDir::FromEnd, 1, INDEX_NONE);
	RunTest(ABACADAB, B, ESearchCase::CaseSensitive, ESearchDir::FromEnd, 0, INDEX_NONE);

	RunTest(ABACADAB, a, ESearchCase::IgnoreCase, ESearchDir::FromStart, 0, 0);
	RunTest(ABACADAB, a, ESearchCase::IgnoreCase, ESearchDir::FromStart, 1, 2);
	RunTest(ABACADAB, a, ESearchCase::IgnoreCase, ESearchDir::FromStart, 7, INDEX_NONE);
	RunTest(ABACADAB, a, ESearchCase::IgnoreCase, ESearchDir::FromEnd, 8, 6);
	RunTest(ABACADAB, a, ESearchCase::IgnoreCase, ESearchDir::FromEnd, 1, 0);
	RunTest(ABACADAB, a, ESearchCase::IgnoreCase, ESearchDir::FromEnd, 0, INDEX_NONE);

	RunTest(ABACADAB, a, ESearchCase::CaseSensitive, ESearchDir::FromStart, 0, INDEX_NONE);
	RunTest(ABACADAB, a, ESearchCase::CaseSensitive, ESearchDir::FromStart, 1, INDEX_NONE);
	RunTest(ABACADAB, a, ESearchCase::CaseSensitive, ESearchDir::FromStart, 7, INDEX_NONE);
	RunTest(ABACADAB, a, ESearchCase::CaseSensitive, ESearchDir::FromEnd, 8, INDEX_NONE);
	RunTest(ABACADAB, a, ESearchCase::CaseSensitive, ESearchDir::FromEnd, 1, INDEX_NONE);
	RunTest(ABACADAB, a, ESearchCase::CaseSensitive, ESearchDir::FromEnd, 0, INDEX_NONE);

	RunTest(ABACADAB, CAD, ESearchCase::CaseSensitive, ESearchDir::FromStart, 0, 3);
	RunTest(ABACADAB, CAD, ESearchCase::CaseSensitive, ESearchDir::FromStart, 3, 3);
	RunTest(ABACADAB, CAD, ESearchCase::CaseSensitive, ESearchDir::FromStart, 4, INDEX_NONE);

	RunTest(ABACADAB, EmptyString, ESearchCase::CaseSensitive, ESearchDir::FromStart, 0, 0);
	RunTest(ABACADAB, EmptyString, ESearchCase::CaseSensitive, ESearchDir::FromStart, 4, 4);
	RunTest(ABACADAB, EmptyString, ESearchCase::CaseSensitive, ESearchDir::FromEnd, 8, 7);
	RunTest(ABACADAB, EmptyString, ESearchCase::CaseSensitive, ESearchDir::FromEnd, 2, 1);
	RunTest(ABACADAB, EmptyString, ESearchCase::CaseSensitive, ESearchDir::FromEnd, 0, INDEX_NONE);

	RunTest(ABACADAB, EmptyString, ESearchCase::IgnoreCase, ESearchDir::FromStart, 0, 0);
	RunTest(ABACADAB, EmptyString, ESearchCase::IgnoreCase, ESearchDir::FromStart, 4, 4);
	RunTest(ABACADAB, EmptyString, ESearchCase::IgnoreCase, ESearchDir::FromEnd, 8, 7);
	RunTest(ABACADAB, EmptyString, ESearchCase::IgnoreCase, ESearchDir::FromEnd, 2, 1);
	RunTest(ABACADAB, EmptyString, ESearchCase::IgnoreCase, ESearchDir::FromEnd, 0, INDEX_NONE);

	// Find with a null char*
	int32 Actual = FString(ABACADAB).Find(nullptr, ESearchCase::CaseSensitive, ESearchDir::FromStart, 0);
	if (Actual != INDEX_NONE)
	{
		AddError(FString::Printf(TEXT("FString(\"ABACADAB\").Find(nullptr, 0, %d, %d, %d) returned Actual %d not equal to Expected -1"),
			(int32)ESearchCase::CaseSensitive, (int32)ESearchDir::FromStart, 0, Actual));
	}
	Actual = FString(ABACADAB).Find(nullptr, ESearchCase::CaseSensitive, ESearchDir::FromEnd, 0);
	if (Actual != INDEX_NONE)
	{
		AddError(FString::Printf(TEXT("FString(\"ABACADAB\").Find(nullptr, 0, %d, %d, %d) returned Actual %d not equal to Expected -1"),
			(int32)ESearchCase::CaseSensitive, (int32)ESearchDir::FromEnd, 0, Actual));
	}

	// Find with a null char* and a length
	Actual = FString(ABACADAB).Find(nullptr, 0, ESearchCase::CaseSensitive, ESearchDir::FromStart, 0);
	if (Actual != 0)
	{
		AddError(FString::Printf(TEXT("FString(\"ABACADAB\").Find(nullptr, 0, %d, %d, %d) returned Actual %d not equal to Expected 0"),
			(int32)ESearchCase::CaseSensitive, (int32)ESearchDir::FromStart, 0, Actual));
	}
	Actual = FString(ABACADAB).Find(nullptr, 0, ESearchCase::CaseSensitive, ESearchDir::FromEnd, 8);
	if (Actual != 7)
	{
		AddError(FString::Printf(TEXT("FString(\"ABACADAB\").Find(nullptr, 0, %d, %d, %d) returned Actual %d not equal to Expected 7"),
			(int32)ESearchCase::CaseSensitive, (int32)ESearchDir::FromEnd, 0, Actual));
	}
	// Negative SubStrLen are not allowed so we do not test them

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FStringConstructWithSlackTest, "System.Core.String.ConstructWithSlack", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool FStringConstructWithSlackTest::RunTest(const FString& Parameters)
{
	auto DoTest = [this](const TCHAR* Ptr, int32 ExtraSlack, const TArray<TCHAR>& Expected)
	{
		FString Str = FString::ConstructWithSlack(Ptr, ExtraSlack);

		const TArray<TCHAR>& StrArr = Str.GetCharArray();
		int32 ActualSlack = StrArr.Max() - StrArr.Num();
		bool  bValidSlack = (ExtraSlack == 0 && *Ptr == TEXT('\0')) ? (ActualSlack == 0) : (ActualSlack >= ExtraSlack);
		if (StrArr != Expected || !bValidSlack)
		{
			AddError(
				FString::Printf(
					TEXT("FString::ConstructWithSlack(TEXT(\"%s\"), %d) failure: result '%s' (expected '%.*s'), slack '%d' (expected '%d')"),
					Ptr,
					ExtraSlack,
					*Str,
					Expected.Num(),
					Expected.GetData(),
					ActualSlack,
					ExtraSlack
				)
			);
		}
	};

	DoTest(TEXT("\0abc"), 0,  {});
	DoTest(TEXT("\0abc"), 47, {});
	DoTest(TEXT("abc"),   0,  { TEXT('a'), TEXT('b'), TEXT('c'), TEXT('\0') });
	DoTest(TEXT("abc"),   47, { TEXT('a'), TEXT('b'), TEXT('c'), TEXT('\0') });

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FStringConstructFromPtrSizeTest, "System.Core.String.ConstructFromPtrSize", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool FStringConstructFromPtrSizeTest::RunTest(const FString& Parameters)
{
	auto DoTest = [this](const TCHAR* Ptr, int32 Size, const TArray<TCHAR>& Expected)
	{
		FString Str = FString::ConstructFromPtrSize(Ptr, Size);

		const TArray<TCHAR>& StrArr = Str.GetCharArray();
		if (StrArr != Expected)
		{
			AddError(
				FString::Printf(
					TEXT("FString::ConstructFromPtrSize(TEXT(\"%s\"), %d) failure: result '%s' (expected '%.*s')"),
					Ptr,
					Size,
					*Str,
					Expected.Num(),
					Expected.GetData())
			);
		}
	};

	DoTest(TEXT("\0abc"),    4, { TEXT('\0'), TEXT('a'), TEXT('b'), TEXT('c'),                                               TEXT('\0') });
	DoTest(TEXT("abc\0def"), 3, { TEXT('a'),  TEXT('b'), TEXT('c'),                                                          TEXT('\0') });
	DoTest(TEXT("abc\0def"), 4, { TEXT('a'),  TEXT('b'), TEXT('c'), TEXT('\0'),                                              TEXT('\0') });
	DoTest(TEXT("abc\0def"), 7, { TEXT('a'),  TEXT('b'), TEXT('c'), TEXT('\0'), TEXT('d'), TEXT('e'), TEXT('f'),             TEXT('\0') });
	DoTest(TEXT("abc\0def"), 8, { TEXT('a'),  TEXT('b'), TEXT('c'), TEXT('\0'), TEXT('d'), TEXT('e'), TEXT('f'), TEXT('\0'), TEXT('\0') });

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FStringConstructFromPtrSizeWithSlackTest, "System.Core.String.ConstructFromPtrSizeWithSlack", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool FStringConstructFromPtrSizeWithSlackTest::RunTest(const FString& Parameters)
{
	auto DoTest = [this](const TCHAR* Ptr, int32 Size, int32 ExtraSlack, const TArray<TCHAR>& Expected)
	{
		FString Str = FString::ConstructFromPtrSizeWithSlack(Ptr, Size, ExtraSlack);

		const TArray<TCHAR>& StrArr = Str.GetCharArray();
		int32 ActualSlack = StrArr.Max() - StrArr.Num();
		bool  bValidSlack = (ActualSlack >= ExtraSlack);
		if (StrArr != Expected || !bValidSlack)
		{
			AddError(
				FString::Printf(
					TEXT("FString::ConstructFromPtrSizeWithSlack(TEXT(\"%s\"), %d) failure: result '%s' (expected '%.*s'), slack '%d' (expected '%d')"),
					Ptr,
					Size,
					*Str,
					Expected.Num(),
					Expected.GetData(),
					ActualSlack,
					ExtraSlack
				)
			);
		}
	};

	DoTest(TEXT("\0abc"),    4, 0,  { TEXT('\0'), TEXT('a'), TEXT('b'), TEXT('c'),                                               TEXT('\0') });
	DoTest(TEXT("\0abc"),    4, 47, { TEXT('\0'), TEXT('a'), TEXT('b'), TEXT('c'),                                               TEXT('\0') });
	DoTest(TEXT("abc\0def"), 3, 0,  { TEXT('a'),  TEXT('b'), TEXT('c'),                                                          TEXT('\0') });
	DoTest(TEXT("abc\0def"), 3, 47, { TEXT('a'),  TEXT('b'), TEXT('c'),                                                          TEXT('\0') });
	DoTest(TEXT("abc\0def"), 4, 0,  { TEXT('a'),  TEXT('b'), TEXT('c'), TEXT('\0'),                                              TEXT('\0') });
	DoTest(TEXT("abc\0def"), 4, 47, { TEXT('a'),  TEXT('b'), TEXT('c'), TEXT('\0'),                                              TEXT('\0') });
	DoTest(TEXT("abc\0def"), 7, 0,  { TEXT('a'),  TEXT('b'), TEXT('c'), TEXT('\0'), TEXT('d'), TEXT('e'), TEXT('f'),             TEXT('\0') });
	DoTest(TEXT("abc\0def"), 7, 47, { TEXT('a'),  TEXT('b'), TEXT('c'), TEXT('\0'), TEXT('d'), TEXT('e'), TEXT('f'),             TEXT('\0') });
	DoTest(TEXT("abc\0def"), 8, 0,  { TEXT('a'),  TEXT('b'), TEXT('c'), TEXT('\0'), TEXT('d'), TEXT('e'), TEXT('f'), TEXT('\0'), TEXT('\0') });
	DoTest(TEXT("abc\0def"), 8, 47, { TEXT('a'),  TEXT('b'), TEXT('c'), TEXT('\0'), TEXT('d'), TEXT('e'), TEXT('f'), TEXT('\0'), TEXT('\0') });

	return true;
}

#endif // WITH_DEV_AUTOMATION_TESTS
