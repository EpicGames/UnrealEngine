// Copyright Epic Games, Inc. All Rights Reserved.

#include "Containers/Ticker.h"
#include "AutoRTFM.h"
#include "Stats/Stats.h"
#include "Misc/TimeGuard.h"

FTSTicker& FTSTicker::GetCoreTicker()
{
	static FTSTicker CoreTicker;
	return CoreTicker;
}

FTSTicker::FDelegateHandle FTSTicker::AddTicker(const FTickerDelegate& InDelegate, float InDelay)
{
	FElementPtr NewElement{ new FElement{ CurrentTime.load(std::memory_order_relaxed) + InDelay, InDelay, InDelegate } };
	AutoRTFM::OnCommit([this, NewElement]
	{
		AddedElements.Enqueue(NewElement); 
	});
	return NewElement;
}

FTSTicker::FDelegateHandle FTSTicker::AddTicker(const TCHAR* InName, float InDelay, TUniqueFunction<bool(float)>&& InFunction)
{
	FElementPtr NewElement{ new FElement{ CurrentTime.load(std::memory_order_relaxed) + InDelay, InDelay, MoveTemp(InFunction) } };
	AutoRTFM::OnCommit([this, NewElement]
	{
		AddedElements.Enqueue(NewElement); 
	});
	return NewElement;
}

uint32 GetThreadId(uint64 State)
{
	return (uint32)(State >> 32);
}

void FTSTicker::RemoveTicker(FDelegateHandle Handle)
{
	AutoRTFM::OnCommit([Handle]
	{
		if (FElementPtr Element = Handle.Pin())
		{
			// mark the element as removed and if it's being ticked atm, spin-wait until its execution is finished
			uint64 PrevState = Element->State.fetch_or(FElement::RemovedState, std::memory_order_acquire); // "acquire" to prevent potential 
			// resource release after RemoveTicker() to be reordered before it
			uint32 ExecutingThreadId = GetThreadId(PrevState);
			
			while (ExecutingThreadId != 0 && // is being executed right now
				FPlatformTLS::GetCurrentThreadId() != ExecutingThreadId) // and is not removed from inside its execution
			{
				FPlatformProcess::Yield();
				ExecutingThreadId = GetThreadId(Element->State.load(std::memory_order_relaxed));
			}
		}
	});
}

void FTSTicker::Tick(float DeltaTime)
{
	SCOPE_TIME_GUARD(TEXT("FTSTicker::Tick"));

	QUICK_SCOPE_CYCLE_COUNTER(STAT_FTicker_Tick);

	const uint64 CurrentThreadId = ((uint64)FPlatformTLS::GetCurrentThreadId()) << 32; // already prepared to be used in `FElement::State`

	// take in all new elements
	auto PumpAddedElementsQueue = [this]() {
		while (TOptional<FElementPtr> AddedElement = AddedElements.Dequeue())
		{
			uint64 State = AddedElement.GetValue()->State.load(std::memory_order_relaxed);
			checkf(GetThreadId(State) == 0, TEXT("Invalid state %" UINT64_FMT), State);
			if (State == FElement::DefaultState)
			{
				Elements.Add(MoveTemp(AddedElement.GetValue()));
			}
		}
	};

	PumpAddedElementsQueue();

	if (!Elements.Num())
	{
		return;
	}

	// We can do a relaxed read/store since only the game thread will call the tick function.
	CurrentTime.store(CurrentTime.load(std::memory_order_relaxed) + DeltaTime, std::memory_order_relaxed);

	TArray<FElementPtr> TickedElements;
	int32 ElementIdx = 0;
	// ticking delegates can add more tickers that must be executed in the same tick call to be backward compatible with the old
	// implementation. keep transfering new tickers to the main list and executing them
	do
	{
		for (; ElementIdx < Elements.Num(); ++ElementIdx)
		{
			FElementPtr& Element = Elements[ElementIdx];
			// set the execution state
			uint64 PrevState = Element->State.fetch_add(CurrentThreadId, std::memory_order_acquire); // "acquire" to prevent anything between this and `ClearExecutionFlag()` to be reordered outside of this scope, is coupled with "release" in `ClearExecutionFlag`
			checkf(GetThreadId(PrevState) == 0, TEXT("Invalid state %" UINT64_FMT), PrevState);
			auto ClearExecutionFlag = [CurrentThreadId](FElementPtr& Element)
			{
				return (uint8)(Element->State.fetch_sub(CurrentThreadId, std::memory_order_release) - CurrentThreadId);
			};

			if (PrevState == FElement::RemovedState)
			{
				ClearExecutionFlag(Element);
				continue;
			}

			if (Element->FireTime > CurrentTime.load(std::memory_order_relaxed))
			{
				ClearExecutionFlag(Element);
				TickedElements.Add(MoveTemp(Element));
				continue;
			}

			if (Element->Fire(DeltaTime))
			{
				PrevState = ClearExecutionFlag(Element); // it can be removed during execution
				if (PrevState != FElement::RemovedState)
				{
					checkf(PrevState == FElement::DefaultState, TEXT("Invalid state %" UINT64_FMT), PrevState);
					Element->FireTime = CurrentTime.load(std::memory_order_relaxed) + Element->DelayTime;
					TickedElements.Add(MoveTemp(Element));
				}
			}
			else
			{
				PrevState = ClearExecutionFlag(Element);
				checkf(PrevState == FElement::DefaultState || PrevState == FElement::RemovedState, TEXT("Invalid state %" UINT64_FMT), PrevState);
			}
		}

		// See if there were new elements added while ticking. If so, tick them this frame as well
		PumpAddedElementsQueue();
	} while (ElementIdx < Elements.Num());

	Elements.Reset();
	Exchange(TickedElements, Elements);
}

void FTSTicker::Reset()
{
	while (AddedElements.Dequeue())
	{
	}
	CurrentTime = 0.0;
	Elements.Reset();
}

FTSTicker::FElement::FElement()
	: FireTime(0.0)
	, DelayTime(0.0f)
{}

FTSTicker::FElement::FElement(double InFireTime, float InDelayTime, const FTickerDelegate& InDelegate)
	: FireTime(InFireTime)
	, DelayTime(InDelayTime)
	, Function(
		[InDelegate](float DeltaTime)
		{ 
			return InDelegate.IsBound() && InDelegate.Execute(DeltaTime);
		}
	)
{}

FTSTicker::FElement::FElement(double InFireTime, float InDelayTime, TUniqueFunction<bool(float)>&& InFunction)
	: FireTime(InFireTime)
	, DelayTime(InDelayTime)
	, Function(MoveTemp(InFunction))
{}

bool FTSTicker::FElement::Fire(float DeltaTime)
{
	return Function(DeltaTime);
}

FTSTickerObjectBase::FTSTickerObjectBase(float InDelay, FTSTicker& InTicker)
{
	// Register delegate for ticker callback
	FTickerDelegate TickDelegate = FTickerDelegate::CreateRaw(this, &FTSTickerObjectBase::Tick);
	TickHandle = InTicker.AddTicker(TickDelegate, InDelay);
}

FTSTickerObjectBase::~FTSTickerObjectBase()
{
	FTSTicker::RemoveTicker(TickHandle);
}
