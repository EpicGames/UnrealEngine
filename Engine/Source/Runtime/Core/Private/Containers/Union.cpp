// Copyright Epic Games, Inc. All Rights Reserved.

#include "Containers/Union.h"

DEFINE_LOG_CATEGORY(LogUnion);
