// Copyright Epic Games, Inc. All Rights Reserved.

#include "Containers/Utf8String.h"
#include "Containers/StringIncludes.cpp.inl"

#define UE_STRING_CLASS             FUtf8String
#define UE_STRING_CHARTYPE          UTF8CHAR
#define UE_STRING_CHARTYPE_IS_TCHAR 0
	#include "Containers/String.cpp.inl"
#undef UE_STRING_CHARTYPE_IS_TCHAR
#undef UE_STRING_CHARTYPE
#undef UE_STRING_CLASS
