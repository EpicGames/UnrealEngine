// Copyright Epic Games, Inc. All Rights Reserved.

#include "Delegates/DelegateBase.h"

#if UE_DELEGATE_CHECK_LIFETIME
#include "Containers/TransactionallySafeMpscQueue.h"
#include "Misc/ScopeLock.h"
#include "Misc/TransactionallySafeCriticalSection.h"
#include "Modules/ModuleManager.h"

DEFINE_LOG_CATEGORY_STATIC(LogDelegates, Log, All);

using FDelegatesRegistrationsToCallingFunctions = TMap<FDelegateHandle, FName>;
using FMulticastRegistrationsPerModule = TMap<FName, FDelegatesRegistrationsToCallingFunctions>;

struct FMulticastDelegateTrackerData
{
	FMulticastRegistrationsPerModule LiveRegistrationsPerModule;
	FTransactionallySafeCriticalSection LiveRegistrationsPerModuleLock;
	TTransactionallySafeMpscQueue<FDelegateHandle> UnregisteredDelegates;
};

FMulticastDelegateTrackerData& GMulticastDelegateTrackerData()
{
	static FMulticastDelegateTrackerData StaticData;
	return StaticData;
}

static bool GEnableDelegateLifetimeChecks = !UE_BUILD_SHIPPING;
static FAutoConsoleVariableRef EnableDelegateLifetimeChecksCVar(
	TEXT("Delegates.EnableDelegateLifetimeChecks"),
	GEnableDelegateLifetimeChecks,
	TEXT("When set, enable the lifetime checking of module-bound delegate instances subscribed to multicast delegates."),
	ECVF_Default);

void FMulticastDelegateTracker::RegisterDelegateInstance(FDelegateHandle DelegateHandle, FName ModuleName, FName CallingFunctionName)
{
	if (ModuleName != NAME_None && GEnableDelegateLifetimeChecks)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(RegisterDelegateInstance);

		FMulticastDelegateTrackerData& TrackerData = GMulticastDelegateTrackerData();

		UE::TScopeLock Lock(TrackerData.LiveRegistrationsPerModuleLock);
		FDelegatesRegistrationsToCallingFunctions& MulticastRegistrations = TrackerData.LiveRegistrationsPerModule.FindOrAdd(ModuleName);
		MulticastRegistrations.Add(DelegateHandle, CallingFunctionName);
	}
}

void FMulticastDelegateTracker::UnregisterDelegateInstance(FDelegateHandle DelegateHandle)
{
	if (GEnableDelegateLifetimeChecks)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UnregisterDelegateInstance);

		GMulticastDelegateTrackerData().UnregisteredDelegates.Enqueue(DelegateHandle);
	}
}

static void RemoveDestroyedMulticastsFromLiveSet()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(RemoveDestroyedMulticastsFromLiveSet);

	FMulticastDelegateTrackerData& TrackerData = GMulticastDelegateTrackerData();
	UE::TScopeLock Lock(TrackerData.LiveRegistrationsPerModuleLock);

	FDelegateHandle DelegateRegistrationToRemove;
	while (TrackerData.UnregisteredDelegates.Dequeue(DelegateRegistrationToRemove))
	{
		for (TPair<FName, FDelegatesRegistrationsToCallingFunctions>& ModuleAndRegistrations : TrackerData.LiveRegistrationsPerModule)
		{
			if (FName* FoundFunctionName = ModuleAndRegistrations.Value.Find(DelegateRegistrationToRemove))
			{
				UE_LOGF(LogDelegates, Verbose, "unregistered from %ls: %ls", *ModuleAndRegistrations.Key.ToString(), *FoundFunctionName->ToString());
				ModuleAndRegistrations.Value.Remove(DelegateRegistrationToRemove);
				break;
			}
		}
	}
}

void FMulticastDelegateTracker::CheckForStaleDelegatesInModules(TConstArrayView<FName> ModuleNames)
{
	if (GEnableDelegateLifetimeChecks)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(CheckForStaleDelegatesInModules);

		RemoveDestroyedMulticastsFromLiveSet();

		FMulticastDelegateTrackerData& TrackerData = GMulticastDelegateTrackerData();
		UE::TScopeLock Lock(TrackerData.LiveRegistrationsPerModuleLock);

		// Now find all live delegate registrations for each of those modules, and report them
		for (FName ModuleName : ModuleNames)
		{
			if (FDelegatesRegistrationsToCallingFunctions* Registrations = TrackerData.LiveRegistrationsPerModule.Find(ModuleName))
			{
				for (const TPair<FDelegateHandle, FName>& DelegateRegistration : *Registrations)
				{
					const FString ReportedMessage = FString::Printf(TEXT("stale delegate in %s: %s"), *ModuleName.ToString(), *DelegateRegistration.Value.ToString());
					UE_LOGF(LogDelegates, Error, "%ls", *ReportedMessage);
					ensureAlwaysMsgf(false, TEXT("%s"), *ReportedMessage);
				}
			}
		}

		// Clear tracking data for every module we've checked so far
		for (FName ModuleName : ModuleNames)
		{
			TrackerData.LiveRegistrationsPerModule.Remove(ModuleName);
		}

#if !UE_BUILD_SHIPPING

		int32 TrackedDelegateInstances = 0;
		size_t AllocatedSize = TrackerData.LiveRegistrationsPerModule.GetAllocatedSize();
		for (const TPair<FName, FDelegatesRegistrationsToCallingFunctions>& ModuleAndRegistrations : TrackerData.LiveRegistrationsPerModule)
		{
			AllocatedSize += ModuleAndRegistrations.Value.GetAllocatedSize();
			TrackedDelegateInstances += ModuleAndRegistrations.Value.Num();
		}

		UE_LOGF(LogDelegates, Verbose, "CheckForStaleDelegatesInModules: %d modules, %d tracked registrations, %dKB used for tracking",
			TrackerData.LiveRegistrationsPerModule.Num(), TrackedDelegateInstances, AllocatedSize / 1024);

#endif // !UE_BUILD_SHIPPING
	}
}

#endif // UE_DELEGATE_CHECK_LIFETIME

void* UE::Core::Private::DelegateAllocate(size_t Size, FDelegateAllocation& Allocation)
{
	int32 NewDelegateSize = FMath::DivideAndRoundUp((int32)Size, (int32)sizeof(FAlignedInlineDelegateType));
	if (Allocation.DelegateSize != NewDelegateSize)
	{
		Allocation.DelegateAllocator.ResizeAllocation(0, NewDelegateSize, sizeof(FAlignedInlineDelegateType));
		Allocation.DelegateSize = NewDelegateSize;
	}

	return Allocation.DelegateAllocator.GetAllocation();
}
