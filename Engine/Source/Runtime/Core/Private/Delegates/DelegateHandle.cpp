// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "Delegates/IDelegateInstance.h"
#include "Templates/Atomic.h"
#include "AutoRTFM.h"

namespace UE::Delegates::Private
{
	TAtomic<uint64> GNextID(1);
}

uint64 FDelegateHandle::GenerateNewID()
{
	// Just increment a counter to generate an ID.
	uint64 Result = 0; // Initialize just to silence static analysis.
	
	UE_AUTORTFM_OPEN
	{
		Result = ++UE::Delegates::Private::GNextID;

		// Check for the next-to-impossible event that we wrap round to 0, because we reserve 0 for null delegates.
		if (Result == 0)
		{
			// Increment it again - it might not be zero, so don't just assign it to 1.
			Result = ++UE::Delegates::Private::GNextID;
		}
	};

	return Result;
}
