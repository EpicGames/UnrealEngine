// Copyright Epic Games, Inc. All Rights Reserved.

#include "Experimental/ConcurrentLinearAllocator.h"


FORCENOINLINE void UE::Core::Private::OnInvalidConcurrentLinearArrayAllocatorNum(int32 NewNum, SIZE_T NumBytesPerElement)
{
	UE_LOG(LogCore, Fatal, TEXT("Trying to resize TConcurrentLinearArrayAllocator to an invalid size of %d with element size %" SIZE_T_FMT), NewNum, NumBytesPerElement);
	for (;;);
}
