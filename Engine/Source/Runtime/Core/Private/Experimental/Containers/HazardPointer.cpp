// Copyright Epic Games, Inc. All Rights Reserved.

#include "Experimental/Containers/HazardPointer.h"

#include "Algo/Sort.h"
#include "Async/UniqueLock.h"
#include "Misc/AppTime.h"

FHazardPointerCollection::~FHazardPointerCollection()
{
	for (FTlsData* Deleters : AllTlsVariables)
	{
		delete Deleters;
	}
	FPlatformTLS::FreeTlsSlot(CollectablesTlsSlot);

	for (FHazardRecordChunk* Block : HazardRecordBlocks)
	{
		delete Block;
	}
}

template<bool Cached>
FHazardPointerCollection::FHazardRecord* FHazardPointerCollection::Grow()
{
	uint32 TotalNumBeforeLock = TotalNumHazardRecords;
	//No empty Entry found: create new
	UE::TUniqueLock Lock(HazardRecordBlocksCS);
	if (TotalNumHazardRecords != TotalNumBeforeLock)
	{
		return Acquire<Cached>();
	}

	FHazardRecordChunk* lst = &Head;
	while (lst->Next.load(std::memory_order_relaxed) != nullptr)
	{
		lst = lst->Next;
	}
	FHazardRecordChunk* p = new FHazardRecordChunk();
	p->Records[0].Retire();
	check(lst->Next.load(std::memory_order_acquire) == nullptr)
	lst->Next.store(p, std::memory_order_release);
	TotalNumHazardRecords += HazardChunkSize; //keep count of HazardPointers, there should not be too many (maybe 2 per thread)

	HazardRecordBlocks.Add(p);
	checkSlow(p->Records[0].GetHazard() == nullptr);
	return &p->Records[0];
}
template CORE_API FHazardPointerCollection::FHazardRecord* FHazardPointerCollection::Grow<true>();
template CORE_API FHazardPointerCollection::FHazardRecord* FHazardPointerCollection::Grow<false>();

void FHazardPointerCollection::Delete(const HazardPointer_Impl::FHazardDeleter& Ptr, int32 CollectLimit)
{
	FTlsData* Collectables = (FTlsData*)FPlatformTLS::GetTlsValue(CollectablesTlsSlot);
	if (Collectables == nullptr)
	{
		UE::TUniqueLock Lock(AllTlsVariablesCS);
		Collectables = new FTlsData();
		AllTlsVariables.Add(Collectables);
		FPlatformTLS::SetTlsValue(CollectablesTlsSlot, Collectables);
	}
	checkSlow(!Collectables->ReclamationList.Contains(Ptr));

	//add to be deleted pointer to thread local List
	Collectables->ReclamationList.Add(Ptr);

	//maybe scan the list
	const double CurrentTime = FAppTime::GetApproximateCurrentTime();
	const bool TimeLimit = (CurrentTime - Collectables->TimeOfLastCollection) > 1.0;
	const int32 DeleteMetric = CollectLimit <= 0 ? ((5 * TotalNumHazardRecords) / 4) : CollectLimit;
	if (TimeLimit || Collectables->ReclamationList.Num() >= DeleteMetric)
	{
		Collectables->TimeOfLastCollection = CurrentTime;
		Collect(Collectables->ReclamationList);
	}
}

static uint32 BinarySearch(const TArray<void*, TInlineAllocator<64>>& Hazards, void* Pointer)
{
	int32 Index = 0;
	int32 Size = Hazards.Num();

	//start with binary search for larger lists
	while (Size > 32)
	{
		const int32 LeftoverSize = Size % 2;
		Size = Size / 2;

		const int32 CheckIndex = Index + Size;
		const int32 IndexIfLess = CheckIndex + LeftoverSize;

		Index = Hazards[CheckIndex] < Pointer ? IndexIfLess : Index;
	}

	//small size array optimization
	int32 ArrayEnd = FMath::Min(Index + Size + 1, Hazards.Num());
	while (Index < ArrayEnd)
	{
		if (Hazards[Index] == Pointer)
		{
			return Index;
		}
		Index++;
	}

	return ~0u;
}

void FHazardPointerCollection::Collect(TArray<HazardPointer_Impl::FHazardDeleter>& Collectables)
{
	TArray<void*, TInlineAllocator<64>> Hazards;
	
	//collect all global Hazards in the System.
	FHazardRecordChunk* p = &Head;
	do
	{
		for (uint32 i = 0; i < HazardChunkSize; i++)
		{
			void* h = p->Records[i].GetHazard();
			if (h && h != reinterpret_cast<void*>(FHazardRecord::FreeHazardEntry))
			{
				Hazards.Add(h);
			};
		}
		p = p->Next;
	} while (p);

	//Sort the pointers for ease of use
	if (Hazards.Num() > 1)
	{
		TArrayRange<void*> ArrayRange(&Hazards[0], Hazards.Num());
		Algo::Sort(ArrayRange, TLess<void*>());
	}

	TArray<HazardPointer_Impl::FHazardDeleter, TInlineAllocator<64>> DeletedCollectables;
	//check all thread local to be deleted pointers if they are NOT in the hazard list
	for (int32 c = 0; c < Collectables.Num(); c++)
	{
		const HazardPointer_Impl::FHazardDeleter& Collectable = Collectables[c];
		uint32 Index = BinarySearch(Hazards, Collectable.Pointer);

		//potentially delete and remove item from thread local List.
		if (Index == ~0u)
		{
			DeletedCollectables.Add(Collectable);
			Collectables[c] = Collectables.Last();
			Collectables.Pop(EAllowShrinking::No);
			c--;
		}
	}

	for (HazardPointer_Impl::FHazardDeleter& DeletedCollectable : DeletedCollectables)
	{
		DeletedCollectable.Delete();
	}
}
