// Copyright Epic Games, Inc. All Rights Reserved.

#include "Experimental/IO/IoStatusError.h"
#include "IO/IoStatus.h"
#include "Containers/AnsiString.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"

UE_DEFINE_ERROR_MODULE(UE::IoStore);

UE_DEFINE_ERROR(UE::IoStore, Unknown);
UE_DEFINE_ERROR(UE::IoStore, InvalidCode);
UE_DEFINE_ERROR(UE::IoStore, Cancelled);
UE_DEFINE_ERROR(UE::IoStore, FileOpenFailed);
UE_DEFINE_ERROR(UE::IoStore, FileNotOpen);
UE_DEFINE_ERROR(UE::IoStore, ReadError);
UE_DEFINE_ERROR(UE::IoStore, WriteError);
UE_DEFINE_ERROR(UE::IoStore, NotFound);
UE_DEFINE_ERROR(UE::IoStore, CorruptToc);
UE_DEFINE_ERROR(UE::IoStore, UnknownChunkID);
UE_DEFINE_ERROR(UE::IoStore, InvalidParameter);
UE_DEFINE_ERROR(UE::IoStore, SignatureError);
UE_DEFINE_ERROR(UE::IoStore, InvalidEncryptionKey);
UE_DEFINE_ERROR(UE::IoStore, CompressionError);
UE_DEFINE_ERROR(UE::IoStore, PendingFork);
UE_DEFINE_ERROR(UE::IoStore, PendingEncryptionKey);
UE_DEFINE_ERROR(UE::IoStore, Disabled);
UE_DEFINE_ERROR(UE::IoStore, NotInstalled);
UE_DEFINE_ERROR(UE::IoStore, PendingHostGroup);
UE_DEFINE_ERROR(UE::IoStore, Timeout);
UE_DEFINE_ERROR(UE::IoStore, DeleteError);
UE_DEFINE_ERROR(UE::IoStore, OutOfDiskSpace);
UE_DEFINE_ERROR(UE::IoStore, FileSeekFailed);
UE_DEFINE_ERROR(UE::IoStore, FileFlushFailed);
UE_DEFINE_ERROR(UE::IoStore, FileMoveFailed);
UE_DEFINE_ERROR(UE::IoStore, FileCloseFailed);
UE_DEFINE_ERROR(UE::IoStore, FileCorrupt);
UE_DEFINE_ERROR(UE::IoStore, HttpReadError);

namespace UE::IoStore
{
	UE::UnifiedError::FError ConvertError(const FIoStatus& Status)
	{
		switch (Status.GetErrorCode())
		{
			default:
				ensureMsgf(false, TEXT("Unhandled IoStatus code %d: Make sure all status codes have a matching UE_DECLARE_ERROR declaration"), (int32)Status.GetErrorCode());
			case EIoErrorCode::Ok:
			case EIoErrorCode::Unknown:
				return UE::IoStore::Unknown(Status.GetErrorMessage());
			case EIoErrorCode::InvalidCode:
				return UE::IoStore::InvalidCode(Status.GetErrorMessage());
			case EIoErrorCode::Cancelled:
				return UE::IoStore::Cancelled(Status.GetErrorMessage());
			case EIoErrorCode::FileOpenFailed:
				return UE::IoStore::FileOpenFailed(Status.GetErrorMessage());
			case EIoErrorCode::FileNotOpen:
				return UE::IoStore::FileNotOpen(Status.GetErrorMessage());
			case EIoErrorCode::ReadError:
				return UE::IoStore::ReadError(Status.GetErrorMessage());
			case EIoErrorCode::WriteError:
				return UE::IoStore::WriteError(Status.GetErrorMessage());
			case EIoErrorCode::NotFound:
				return UE::IoStore::NotFound(Status.GetErrorMessage());
			case EIoErrorCode::CorruptToc:
				return UE::IoStore::CorruptToc(Status.GetErrorMessage());
			case EIoErrorCode::UnknownChunkID:
				return UE::IoStore::UnknownChunkID(Status.GetErrorMessage());
			case EIoErrorCode::InvalidParameter:
				return UE::IoStore::InvalidParameter(Status.GetErrorMessage());
			case EIoErrorCode::SignatureError:
				return UE::IoStore::SignatureError(Status.GetErrorMessage());
			case EIoErrorCode::InvalidEncryptionKey:
				return UE::IoStore::InvalidEncryptionKey( Status.GetErrorMessage());
			case EIoErrorCode::CompressionError:
				return UE::IoStore::CompressionError(Status.GetErrorMessage());
			case EIoErrorCode::PendingFork:
				return UE::IoStore::PendingFork(Status.GetErrorMessage());
			case EIoErrorCode::PendingEncryptionKey:
				return UE::IoStore::PendingEncryptionKey( Status.GetErrorMessage());
			case EIoErrorCode::Disabled:
				return UE::IoStore::Disabled(Status.GetErrorMessage());
			case EIoErrorCode::NotInstalled:
				return UE::IoStore::NotInstalled(Status.GetErrorMessage());
			case EIoErrorCode::PendingHostGroup:
				return UE::IoStore::PendingHostGroup(Status.GetErrorMessage());
			case EIoErrorCode::Timeout:
				return UE::IoStore::Timeout(Status.GetErrorMessage());
			case EIoErrorCode::DeleteError:
				return UE::IoStore::DeleteError(Status.GetErrorMessage());
			case EIoErrorCode::OutOfDiskSpace:
				return UE::IoStore::OutOfDiskSpace(Status.GetErrorMessage());
			case EIoErrorCode::FileSeekFailed:
				return UE::IoStore::FileSeekFailed(Status.GetErrorMessage());
			case EIoErrorCode::FileFlushFailed:
				return UE::IoStore::FileFlushFailed(Status.GetErrorMessage());
			case EIoErrorCode::FileMoveFailed:
				return UE::IoStore::FileMoveFailed(Status.GetErrorMessage());
			case EIoErrorCode::FileCloseFailed:
				return UE::IoStore::FileCloseFailed(Status.GetErrorMessage());
			case EIoErrorCode::FileCorrupt:
				return UE::IoStore::FileCorrupt(Status.GetErrorMessage());
			case EIoErrorCode::HttpReadError:
				return UE::IoStore::HttpReadError(Status.GetErrorMessage());
		}
	}

	void StripInvalidErrorCodeCharacters(FUtf8String& ErrorCode)
	{
		ErrorCode.ReplaceCharInline(UTF8TEXT(' '), UTF8TEXT('_'));
	}
}