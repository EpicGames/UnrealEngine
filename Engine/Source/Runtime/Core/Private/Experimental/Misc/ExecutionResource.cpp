// Copyright Epic Games, Inc. All Rights Reserved.
#include "Experimental/Misc/ExecutionResource.h"

#include "Containers/Array.h"
#include "HAL/Platform.h"
#include "Templates/UnrealTemplate.h"
#include "Async/Fundamental/Task.h"
#include "Containers/Map.h"

namespace ExecutionResourceImpl
{
	class FState
	{
	public:
		void Push(const LowLevelTasks::FTask* ActiveTask, TRefCountPtr<IExecutionResource> ExecutionResource)
		{
			if (MainTask == INVALID_TASK || MainTask == UPTRINT(ActiveTask))
			{
				MainTask = UPTRINT(ActiveTask);
				MainStack.Push(MoveTemp(ExecutionResource));
			}
			else
			{
				AdditionalStacks.FindOrAdd(ActiveTask).Push(MoveTemp(ExecutionResource));
			}
		}

		void Pop(const LowLevelTasks::FTask* ActiveTask)
		{
			if (MainTask == UPTRINT(ActiveTask))
			{
				MainStack.Pop(EAllowShrinking::No);
				if (MainStack.IsEmpty())
				{
					MainTask = INVALID_TASK;
					MainStack.Shrink();
				}
			}
			else
			{
				TArray<TRefCountPtr<IExecutionResource>>& Stack = AdditionalStacks.FindChecked(ActiveTask);
				Stack.Pop();
				if (Stack.IsEmpty())
				{
					AdditionalStacks.Remove(ActiveTask);
				}
			}
		}

		TArray<TRefCountPtr<IExecutionResource>>* Get(const LowLevelTasks::FTask* ActiveTask)
		{
			if (MainTask == UPTRINT(ActiveTask))
			{
				return &MainStack;
			}
			else
			{
				return AdditionalStacks.Find(ActiveTask);
			}
		}

	private:
		// Define the invalid task as something other than nullptr for the case when we're not called as part of a backend task.
		static constexpr UPTRINT INVALID_TASK = 1;

		// Fast path to avoid using the map in the normal case
		UPTRINT                                  MainTask = INVALID_TASK;
		TArray<TRefCountPtr<IExecutionResource>> MainStack;

		// Will be used during busy waits (i.e. multiple active task for the same thread_local)
		// Because we don't want busy wait task to end up with the resource of another task.
		TMap<const LowLevelTasks::FTask*, TArray<TRefCountPtr<IExecutionResource>>> AdditionalStacks;
	};

	static thread_local FState State;
}

FExecutionResourceContextScope::FExecutionResourceContextScope(TRefCountPtr<IExecutionResource> ExecutionResource)
{
	ExecutionResourceImpl::State.Push(LowLevelTasks::FTask::GetActiveTask(), MoveTemp(ExecutionResource));
}

FExecutionResourceContextScope::~FExecutionResourceContextScope()
{
	ExecutionResourceImpl::State.Pop(LowLevelTasks::FTask::GetActiveTask());
}

class FCompositeExecutionResource : public IExecutionResource, public FRefCountedObject
{
public:
	FCompositeExecutionResource(const TArray<TRefCountPtr<IExecutionResource>>& InExecutionResources)
		: ExecutionResources(InExecutionResources)
	{
	}

	void AddRef() const override
	{
		return FRefCountedObject::AddRef();
	}

	FReturnedRefCountValue Release() const override
	{
		return FRefCountedObject::Release();
	}
	
	FReturnedRefCountValue GetRefCount() const override
	{
		return FRefCountedObject::GetRefCount();
	}

private:
	TArray<TRefCountPtr<IExecutionResource>> ExecutionResources;
};

TRefCountPtr<IExecutionResource> FExecutionResourceContext::Get()
{
	TArray<TRefCountPtr<IExecutionResource>>* Stack = ExecutionResourceImpl::State.Get(LowLevelTasks::FTask::GetActiveTask());

	if (Stack == nullptr || Stack->IsEmpty())
	{
		return nullptr;
	}

	return new FCompositeExecutionResource(*Stack);
}