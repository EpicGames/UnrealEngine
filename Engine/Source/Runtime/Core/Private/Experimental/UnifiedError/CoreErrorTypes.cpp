// Copyright Epic Games, Inc. All Rights Reserved.

#include "Experimental/UnifiedError/CoreErrorTypes.h"

UE_DEFINE_ERROR_MODULE(UE::Core);
UE_DEFINE_ERROR(UE::Core, ArgumentError);
UE_DEFINE_ERROR(UE::Core, CancellationError);

namespace UE::Core
{

bool IsCancellationError(const UE::UnifiedError::FError& Error)
{
	return Error == UE::Core::CancellationError;
}

} // namespace UE::Core

