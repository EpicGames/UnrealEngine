// Copyright Epic Games, Inc. All Rights Reserved.

#include "Experimental/UnifiedError/UnifiedError.h"
#include "Algo/Reverse.h"
#include "Containers/AnsiString.h"
#include "Internationalization/Text.h"
#include "Internationalization/Internationalization.h"
#include "Serialization/CompactBinarySerialization.h"
#include "Logging/StructuredLog.h"
#include "Logging/StructuredLogFormat.h"


namespace UE::UnifiedError
{

// FError functions


void FError::Invalidate()
{
	// Invert the error code to help track usage after MoveTemp
	// Leave module code unmodified.
	ErrorCode = -ErrorCode;
	ErrorDetails = nullptr;
}

const FMandatoryErrorDetails* FError::GetMandatoryErrorDetails() const
{
	TRefCountPtr<const IErrorDetails> CurrentIt = ErrorDetails;
	while (CurrentIt != nullptr)
	{
		TRefCountPtr<const IErrorDetails> Next = CurrentIt->GetInnerErrorDetails();
		if (Next == nullptr)
		{
			break;
		}
		CurrentIt = Next;
	}

	if (!ensureMsgf(CurrentIt.GetReference() != nullptr, TEXT("Attempting to read mandatory error details from an invalid FError, ModuleId:%d ErrorCode:%d"), ModuleId, ErrorCode))
	{
		return nullptr;
	}
	return (const FMandatoryErrorDetails*)(CurrentIt.GetReference());

}

FUtf8StringView FError::GetModuleIdAndErrorCodeString() const
{
	const FMandatoryErrorDetails* MandatoryErrorDetails = GetMandatoryErrorDetails();
	if (!MandatoryErrorDetails)
	{
		// if you are seeing this it's possible this FError was moved and you are using it after the move, or it was constructed in an invalid way (memzero). 
		return FUtf8StringView("Invalid"); 
	}
	return MandatoryErrorDetails->GetModuleIdAndErrorCodeString(*this);
}


FUtf8StringView FError::GetErrorCodeString() const
{
	const FMandatoryErrorDetails* MandatoryErrorDetails = GetMandatoryErrorDetails();
	if (!MandatoryErrorDetails)
	{
		// if you are seeing this it's possible this FError was moved and you are using it after the move, or it was constructed in an invalid way (memzero). 
		return FUtf8StringView("Invalid"); 
	}
	return MandatoryErrorDetails->GetErrorCodeString(*this);
}

FUtf8StringView FError::GetModuleIdString() const
{
	const FMandatoryErrorDetails* MandatoryErrorDetails = GetMandatoryErrorDetails();
	if (!MandatoryErrorDetails)
	{
		// if you are seeing this it's possible this FError was moved and you are using it after the move, or it was constructed in an invalid way (memzero). 
		return FUtf8StringView("Invalid"); 
	}
	return MandatoryErrorDetails->GetModuleIdString(*this);
}



#define USE_LOCALIZED_STRUCTURED_LOG_FOR_FERRORMESSAGE 1

const FText DefaultLogFormat_WithContexts = NSLOCTEXT("UnifiedError", "DefaultLog_WithContexts", "{ModuleIdString}::{ErrorCodeString}: {Details}");
const FText DefaultLogFormat_NoContexts = NSLOCTEXT("UnifiedError", "DefaultLog_NoContexts", "{ModuleIdString}::{ErrorCodeString}: {Details[0]}");
const FText ErrorText_MovedFrom = NSLOCTEXT("UnifiedError", "ErrorText_MovedFrom", "{ModuleId}::{NegatedErrorCode}: Error has been moved from or otherwise invalidated.");

FText FError::CreateErrorMessage(bool bIncludeContext) const
{
	if (!ErrorDetails)
	{
		return FText::FormatNamed(ErrorText_MovedFrom, TEXT("ModuleId"), ModuleId, TEXT("NegatedErrorCode"), ErrorCode);
	}

	EDetailFilter LogDetailFilter = EDetailFilter::IncludeInLogMessage;
	if (bIncludeContext)
	{
		LogDetailFilter |= EDetailFilter::IncludeInContextLogMessage;
	}

	FCbWriter Writer;
	SerializeToCompactBinary(Writer, LogDetailFilter);

	TArray<char, TInlineAllocator<1024>> ScratchBuffer;
	ScratchBuffer.AddUninitialized((int32)Writer.GetSaveSize());
	FCbFieldViewIterator AsCb = Writer.Save(MakeMemoryView(ScratchBuffer));

	FText FormatText = bIncludeContext ? UE::UnifiedError::DefaultLogFormat_WithContexts : UE::UnifiedError::DefaultLogFormat_NoContexts;

#if USE_LOCALIZED_STRUCTURED_LOG_FOR_FERRORMESSAGE
	UE::FLogTemplateOptions TemplateOptions;
	TemplateOptions.bAllowSubObjectReferences = true;

	FInlineLogTemplate Template(FormatText, TemplateOptions);
#else
	FString FormatString = FormatText.ToString();
	UE::FLogTemplateOptions TemplateOptions;
	TemplateOptions.bAllowSubObjectReferences = true;
	FInlineLogTemplate Template(*FormatString, TemplateOptions);
#endif
	// Note that structured log formatting wants a field iterator with the fields used in the format string and not an object
	// And that SerializeToCompactBinary serializes a sequence of fields without wrapping them in an object.
	return Template.FormatToText(AsCb);
}

void FError::SerializeToCompactBinary(FCbWriter& Writer, const EDetailFilter DetailFilter) const
{
	Writer.AddString(UTF8TEXTVIEW("$type"), UTF8TEXTVIEW("UE::UnifiedError::FError"));
	const FMandatoryErrorDetails* MandatoryDetails = SerializeDetails(Writer, DetailFilter, /* bIncludeRoot */ true);
	SerializeLogFormat(Writer, DefaultLogFormat_WithContexts);
	Writer.AddString(UTF8TEXTVIEW("ErrorCodeString"), MandatoryDetails ? MandatoryDetails->GetErrorCodeString(*this) : UTF8TEXTVIEW(""));
	Writer.AddString(UTF8TEXTVIEW("ModuleIdString"), MandatoryDetails ? MandatoryDetails->GetModuleIdString(*this) : UTF8TEXTVIEW(""));
	Writer.AddInteger(UTF8TEXTVIEW("ErrorCode"), ErrorCode);
	Writer.AddInteger(UTF8TEXTVIEW("ModuleId"), ModuleId);
}

const FMandatoryErrorDetails* FError::SerializeDetails(FCbWriter& Writer, const EDetailFilter DetailFilter, bool bIncludeRoot) const
{
	const IErrorDetails* Root = nullptr;
	// Write details in reverse order so that the mandatory details is first 
	TArray<const IErrorDetails*> Details;
	for (const IErrorDetails* DetailsIt = ErrorDetails.GetReference(); DetailsIt != nullptr; DetailsIt = DetailsIt->GetInnerErrorDetails())
	{
		if (DetailsIt->ShouldInclude(DetailFilter))
		{
			Details.Add(DetailsIt);
		}
	}
	Algo::Reverse(Details);
	Writer.BeginArray(ANSITEXTVIEW("Details"));
	for (const IErrorDetails* DetailsIt : Details)
	{
		DetailsIt->SerializeToCompactBinary(Writer);
	}
	Writer.EndArray();
	// Last details must always be a child of FMandatoryErrorDetails on construction, but we may have no details if this object was moved out of 
	return static_cast<const FMandatoryErrorDetails*>(Details.Num() ? Details[0] : nullptr);
}

FString FError::SerializeToJsonString(const EDetailFilter DetailFilter) const
{
	FCbWriter Writer;
	Writer.BeginObject();
	SerializeToCompactBinary(Writer, DetailFilter);
	Writer.EndObject();
	TArray<char, TInlineAllocator<1024>> ScratchBuffer;
	ScratchBuffer.AddUninitialized((int32)Writer.GetSaveSize());
	FCbFieldViewIterator AsCb = Writer.Save(MakeMemoryView(ScratchBuffer));
	TStringBuilder<1024> Text;
	CompactBinaryToCompactJson(AsCb, Text);
	return Text.ToString();
}

FString FError::SerializeToJsonForAnalytics() const
{
	return SerializeToJsonString(EDetailFilter::IncludeInAnalytics);
}

TRefCountPtr<const IErrorDetails> FError::GetErrorDetails(uint64 DeclarationHash) const
{
	for (const IErrorDetails* CurrentIt = ErrorDetails; CurrentIt != nullptr; CurrentIt = CurrentIt->GetInnerErrorDetails())
	{
		if (CurrentIt->GetErrorDetailsTypeId() == DeclarationHash)
		{
			// Add refcount when returning
			return { CurrentIt };
		}
	}
	return nullptr;
}

void SerializeForLog(FCbWriter& Writer, const FError& Error)
{
	Writer.BeginObject();
	Error.SerializeToCompactBinary(Writer, EDetailFilter::IncludeInLogMessage);
	Writer.EndObject();
}

} // namespace UE::UnifiedError

void SerializeForError(FCbWriter& Writer, const FText& Value)
{
	Writer.AddString(Value.ToString());
}

namespace UE::UnifiedError
{

// FErrorDetailsRegistry
void FErrorDetailsRegistry::RegisterDetails(uint64 DeclarationHash, FUtf8StringView ErrorDetailsName, TFunction<IErrorDetails*()> CreationFunction)
{
	CreateFunctions.Add(DeclarationHash, FRegisteredDetails{ .Name = ErrorDetailsName, .CreateFunction = CreationFunction });
}

// ErrorRegistry functionality

namespace ErrorRegistry
{
	class FErrorRegistry
	{
	public:
		FErrorRegistry() {}

	public:
		uint32 RegisterModule(const FAnsiStringView ModuleName);

		int32 RegisterErrorCode(const FAnsiStringView ErrorName, int32 ModuleId, int32 ErrorCode);

	private:
		TMap<int32, FAnsiString> ModuleNameMap;
		TMap<TPair<int32, int32>, FAnsiString> ErrorCodeNameMap;
	};


	uint32 FErrorRegistry::RegisterModule(const FAnsiStringView ModuleName)
	{
		// todo: need to replace this with a stable hashing function
		uint32 ModuleId = GetTypeHash(ModuleName);
		checkf(ModuleNameMap.Contains(ModuleId) == false, TEXT("Module %s and %s are trying to register under module id %d"), ANSI_TO_TCHAR(ModuleName.GetData()), ANSI_TO_TCHAR(*ModuleNameMap.FindRef(ModuleId)), ModuleId);
		ModuleNameMap.Add(ModuleId, ModuleName.GetData());
		return ModuleId;
	}

	int32 FErrorRegistry::RegisterErrorCode(const FAnsiStringView ErrorName, int32 ModuleId, int32 ErrorCode)
	{
		TPair<int32, int32> CombinedErrorId(ModuleId, ErrorCode);
		checkf(ErrorCodeNameMap.Contains(CombinedErrorId) == false, TEXT("Error %s and %s are trying to register under same error code moduleid:%d errorcode:%d"), ANSI_TO_TCHAR(ErrorName.GetData()), ANSI_TO_TCHAR(*ErrorCodeNameMap.FindRef(CombinedErrorId)), ModuleId, ErrorCode);
		ErrorCodeNameMap.Add(CombinedErrorId, ErrorName.GetData());
		return ErrorCode;
	}

	FErrorRegistry GErrorRegistry;

	CORE_API uint32 RegisterModule(const FUtf8StringView ModuleName)
	{
		return GetTypeHash(ModuleName);
		//return GErrorRegistry.RegisterModule(ModuleName);
	}

	CORE_API int32 RegisterErrorCode(const FUtf8StringView ErrorName, int32 ModuleId, int32 ErrorCode)
	{
		return ErrorCode;
		// return GErrorRegistry.RegisterErrorCode(ErrorName, ModuleId, ErrorCode);
	}
}

} // namespace UE::Error

