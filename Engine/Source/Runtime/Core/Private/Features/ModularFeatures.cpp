// Copyright Epic Games, Inc. All Rights Reserved.

#include "Features/ModularFeatures.h"

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "Misc/ScopeLock.h"

IModularFeatures& IModularFeatures::Get()
{
	// Singleton instance
	static FModularFeatures ModularFeatures;
	return ModularFeatures;
}

void FModularFeatures::LockModularFeatureList()
{
	ModularFeaturesMapCriticalSection.Lock();
}

void FModularFeatures::UnlockModularFeatureList()
{
	ModularFeaturesMapCriticalSection.Unlock();
}

int32 FModularFeatures::GetModularFeatureImplementationCount( const FName Type )
{
	// IModularFeature counting is not thread-safe unless wrapped with LockModularFeatureList/UnlockModularFeatureList if you are crashing here

	return ModularFeaturesMap.Num( Type );
}

IModularFeature* FModularFeatures::GetModularFeatureImplementation( const FName Type, const int32 Index )
{
	// IModularFeature fetching is not thread-safe unless wrapped with LockModularFeatureList/UnlockModularFeatureList if you are crashing here

	IModularFeature* ModularFeature = nullptr;

	int32 CurrentIndex = 0;
	for( TMultiMap< FName, class IModularFeature* >::TConstKeyIterator It( ModularFeaturesMap, Type ); It; ++It )
	{
		if( Index == CurrentIndex )
		{
			ModularFeature = It.Value();
			break;
		}

		++CurrentIndex;
	}

	return ModularFeature;
}


void FModularFeatures::RegisterModularFeature( const FName Type, IModularFeature* ModularFeature )
{
	check(ModularFeature != nullptr);
	check(Type.IsValid());

	UE::TScopeLock ScopeLock(ModularFeaturesMapCriticalSection);

	ModularFeaturesMap.AddUnique( Type, ModularFeature );
	ModularFeatureRegisteredEvent.Broadcast( Type, ModularFeature );
}


void FModularFeatures::UnregisterModularFeature( const FName Type, IModularFeature* ModularFeature )
{
	if(ModularFeature == nullptr)
	{
		return;
	}

	check(Type.IsValid());

	UE::TScopeLock ScopeLock(ModularFeaturesMapCriticalSection);

	ModularFeaturesMap.RemoveSingle( Type, ModularFeature );
	ModularFeatureUnregisteredEvent.Broadcast( Type, ModularFeature );
}

IModularFeatures::FOnModularFeatureRegistered& FModularFeatures::OnModularFeatureRegistered()
{
	return ModularFeatureRegisteredEvent;
}

IModularFeatures::FOnModularFeatureUnregistered& FModularFeatures::OnModularFeatureUnregistered()
{
	return ModularFeatureUnregisteredEvent;
}
