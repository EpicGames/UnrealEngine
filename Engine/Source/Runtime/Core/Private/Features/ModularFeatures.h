// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Delegates/Delegate.h"
#include "Features/IModularFeature.h"
#include "Features/IModularFeatures.h"
#include "HAL/CriticalSection.h"  // jira SOL-6812: Remove this.
#include "Misc/TransactionallySafeCriticalSection.h"
#include "UObject/NameTypes.h"

/**
 * Private implementation of modular features interface
 */
class FModularFeatures : public IModularFeatures
{

public:

	/** IModularFeatures interface */
	virtual int32 GetModularFeatureImplementationCount( const FName Type ) override;
	virtual IModularFeature* GetModularFeatureImplementation( const FName Type, const int32 Index ) override;
	virtual void RegisterModularFeature( const FName Type, class IModularFeature* ModularFeature ) override;
	virtual void UnregisterModularFeature( const FName Type, class IModularFeature* ModularFeature ) override;
	DECLARE_DERIVED_EVENT(FModularFeatures, IModularFeatures::FOnModularFeatureRegistered, FOnModularFeatureRegistered);
	virtual IModularFeatures::FOnModularFeatureRegistered& OnModularFeatureRegistered() override;
	DECLARE_DERIVED_EVENT(FModularFeatures, IModularFeatures::FOnModularFeatureUnregistered, FOnModularFeatureUnregistered);
	virtual IModularFeatures::FOnModularFeatureUnregistered& OnModularFeatureUnregistered() override;
	virtual void LockModularFeatureList() override;
	virtual void UnlockModularFeatureList() override;

private:


private:
	/** Maps each feature type to a list of known providers of that feature */
	TMultiMap< FName, class IModularFeature* > ModularFeaturesMap;

	/** Lock modular features map so it can be used across threads, this could be a RW lock instead of a crit sec */
	FTransactionallySafeCriticalSection ModularFeaturesMapCriticalSection;

	/** Event used to inform clients that a modular feature has been registered */
	FOnModularFeatureRegistered ModularFeatureRegisteredEvent;

	/** Event used to inform clients that a modular feature has been unregistered */
	FOnModularFeatureUnregistered ModularFeatureUnregisteredEvent;
};
