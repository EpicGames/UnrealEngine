// Copyright Epic Games, Inc. All Rights Reserved.

#include "FileCache/FileCache.h"
#include "Containers/BinaryHeap.h"
#include "Containers/Queue.h"
#include "Containers/LockFreeList.h"
#include "Templates/TypeHash.h"
#include "Misc/CoreDelegates.h"
#include "Misc/ScopeLock.h"
#include "Async/AsyncFileHandle.h"
#include "Async/TaskGraphInterfaces.h"
#include "ProfilingDebugging/LoadTimeTracker.h"
#include "HAL/PlatformFile.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/IConsoleManager.h"
#include "ProfilingDebugging/CsvProfiler.h"

DECLARE_STATS_GROUP(TEXT("Streaming File Cache"), STATGROUP_SFC, STATCAT_Advanced);

DECLARE_CYCLE_STAT(TEXT("Create Handle"), STAT_SFC_CreateHandle, STATGROUP_SFC);
DECLARE_CYCLE_STAT(TEXT("Read Data"), STAT_SFC_ReadData, STATGROUP_SFC);
DECLARE_CYCLE_STAT(TEXT("EvictAll"), STAT_SFC_EvictAll, STATGROUP_SFC);

// These below are pretty high throughput and probably should be removed once the system gets more mature
DECLARE_CYCLE_STAT(TEXT("Find Eviction Candidate"), STAT_SFC_FindEvictionCandidate, STATGROUP_SFC);

CSV_DEFINE_CATEGORY(FileCache, true);

DEFINE_LOG_CATEGORY_STATIC(LogStreamingFileCache, Log, All);

static int32 GFileCacheBlockSizeKB = 64;
static FAutoConsoleVariableRef CVarFileCacheBlockSize(
	TEXT("fc.BlockSize"),
	GFileCacheBlockSizeKB,
	TEXT("Size of each block in KB in the global file cache object\n"
	     "Should match packaging compression block size for optimal reading from packege"),
	ECVF_ReadOnly
);

static int32 GNumFileCacheBlocks = 64;
static FAutoConsoleVariableRef CVarNumFileCacheBlocks(
	TEXT("fc.NumBlocks"),
	GNumFileCacheBlocks,
	TEXT("Number of blocks in the global file cache object"),
	ECVF_ReadOnly
);

// 
// Strongly typed ids to avoid confusion in the code
// 
template <typename Parameter> class StrongBlockIdentifier
{
	static const int InvalidHandle = 0xFFFFFFFF;

public:
	StrongBlockIdentifier() : Id(InvalidHandle) {}
	explicit StrongBlockIdentifier(int32 SetId) : Id(SetId) {}

	inline bool IsValid() const { return Id != InvalidHandle; }
	inline int32 Get() const { checkSlow(IsValid()); return Id; }

	inline StrongBlockIdentifier& operator++() { Id = Id + 1; return *this; }
	inline StrongBlockIdentifier& operator--() { Id = Id - 1; return *this; }
	inline StrongBlockIdentifier operator++(int) { StrongBlockIdentifier Temp(*this); operator++(); return Temp; }
	inline StrongBlockIdentifier operator--(int) { StrongBlockIdentifier Temp(*this); operator--(); return Temp; }

	// Get the offset in the file to read this block
	inline static int64 GetSize() { return ((int64)GFileCacheBlockSizeKB * 1024); }
	inline int64 GetOffset() const { checkSlow(IsValid()); return (int64)Id * GetSize(); }

	// Get the number of bytes that need to be read for this block
	// takes into account incomplete blocks at the end of the file
	inline int64 GetSize(int64 FileSize) const { checkSlow(IsValid()); return FMath::Min(GetSize(), FileSize - GetOffset()); }

	friend inline uint32 GetTypeHash(const StrongBlockIdentifier<Parameter>& Info) { return GetTypeHash(Info.Id); }

	inline bool operator==(const StrongBlockIdentifier<Parameter>&Other) const { return Id == Other.Id; }
	inline bool operator!=(const StrongBlockIdentifier<Parameter>&Other) const { return Id != Other.Id; }

private:
	int32 Id;
};

using CacheLineID = StrongBlockIdentifier<struct CacheLineStrongType>; // Unique per file handle
using CacheSlotID = StrongBlockIdentifier<struct CacheSlotStrongType>; // Unique per cache

class FFileCacheHandle;

// Some terminology:
// A line: A fixed size block of a file on disc that can be brought into the cache
// Slot: A fixed size piece of memory that can contain the data for a certain line in memory

////////////////

class FFileCache
{
public:
	explicit FFileCache(int32 NumSlots);

	~FFileCache()
	{
		FMemory::Free(Memory);
	}

	uint8* GetSlotMemory(CacheSlotID SlotID)
	{
		check(SlotID.Get() < SlotInfo.Num() - 1);
		check(IsSlotLocked(SlotID)); // slot must be locked in order to access memory
		return Memory + SlotID.Get() * CacheSlotID::GetSize();
	}

	CacheSlotID AcquireAndLockSlot(FFileCacheHandle* InHandle, CacheLineID InLineID);
	bool IsSlotLocked(CacheSlotID InSlotID) const;
	void LockSlot(CacheSlotID InSlotID);
	void UnlockSlot(CacheSlotID InSlotID);

	// if InFile is null, will evict all slots
	bool EvictAll(FFileCacheHandle* InFile = nullptr);

	void FlushCompletedRequests();

	struct FSlotInfo
	{
		FFileCacheHandle* Handle;
		CacheLineID LineID;
		int32 NextSlotIndex;
		int32 PrevSlotIndex;
		int32 LockCount;
	};

	void EvictFileCacheFromConsole()
	{
		EvictAll();
	}

	void PushCompletedRequest(IAsyncReadRequest* Request)
	{
		check(Request);
		CompletedRequests.Push(Request);
		if (((uint32)CompletedRequestsCounter.Increment() % 32u) == 0u)
		{
			FFunctionGraphTask::CreateAndDispatchWhenReady([this]()
			{
				while (IAsyncReadRequest* CompletedRequest = this->CompletedRequests.Pop())
				{
					// Requests are added to this list from the completed callback, but the final completion flag is not set until after callback is finished
					// This means that there's a narrow window where the request is not technically considered to be complete yet
					verify(CompletedRequest->WaitCompletion());
					delete CompletedRequest;
				}
			}, TStatId());
		}
	}

	inline void UnlinkSlot(int32 SlotIndex)
	{
		check(SlotIndex != 0);
		FSlotInfo& Info = SlotInfo[SlotIndex];
		SlotInfo[Info.PrevSlotIndex].NextSlotIndex = Info.NextSlotIndex;
		SlotInfo[Info.NextSlotIndex].PrevSlotIndex = Info.PrevSlotIndex;
		Info.NextSlotIndex = Info.PrevSlotIndex = SlotIndex;
	}

	inline void LinkSlotTail(int32 SlotIndex)
	{
		check(SlotIndex != 0);
		FSlotInfo& HeadInfo = SlotInfo[0];
		FSlotInfo& Info = SlotInfo[SlotIndex];
		check(Info.NextSlotIndex == SlotIndex);
		check(Info.PrevSlotIndex == SlotIndex);

		Info.NextSlotIndex = 0;
		Info.PrevSlotIndex = HeadInfo.PrevSlotIndex;
		SlotInfo[HeadInfo.PrevSlotIndex].NextSlotIndex = SlotIndex;
		HeadInfo.PrevSlotIndex = SlotIndex;
	}

	inline void LinkSlotHead(int32 SlotIndex)
	{
		check(SlotIndex != 0);
		FSlotInfo& HeadInfo = SlotInfo[0];
		FSlotInfo& Info = SlotInfo[SlotIndex];
		check(Info.NextSlotIndex == SlotIndex);
		check(Info.PrevSlotIndex == SlotIndex);

		Info.NextSlotIndex = HeadInfo.NextSlotIndex;
		Info.PrevSlotIndex = 0;
		SlotInfo[HeadInfo.NextSlotIndex].PrevSlotIndex = SlotIndex;
		HeadInfo.NextSlotIndex = SlotIndex;
	}

	FCriticalSection CriticalSection;

	FAutoConsoleCommand EvictFileCacheCommand;

	TLockFreePointerListUnordered<IAsyncReadRequest, PLATFORM_CACHE_LINE_SIZE> CompletedRequests;
	FThreadSafeCounter CompletedRequestsCounter;

	// allocated with an extra dummy entry at index0 for linked list head
	TArray<FSlotInfo> SlotInfo;
	uint8* Memory;
	int64 SizeInBytes;
	int32 NumFreeSlots;
	int32 MinNumFreeSlots;
};

static FFileCache &GetCache()
{
	static FFileCache TheCache(GNumFileCacheBlocks);
	return TheCache;
}

///////////////

class FFileCacheHandle : public IFileCacheHandle
{
public:

	explicit FFileCacheHandle(IAsyncReadFileHandle* InHandle, int64 InBaseOffset);
	virtual ~FFileCacheHandle() override;

	//
	// Block helper functions. These are just convenience around basic math.
	// 

	// templated uses of this may end up converting int64 to int32, but it's up to the user of the template to know
	PRAGMA_DISABLE_UNSAFE_TYPECAST_WARNINGS
	/*
		* Get the block id that contains the specified offset
		*/
	template<typename BlockIDType> inline BlockIDType GetBlock(int64 Offset)
	{
		return BlockIDType(FMath::DivideAndRoundDown(Offset, BlockIDType::GetSize()));
	}
	PRAGMA_RESTORE_UNSAFE_TYPECAST_WARNINGS

	template<typename BlockIDType> inline int32 GetNumBlocks(int64 Offset, int64 Size)
	{
		BlockIDType FirstBlock = GetBlock<BlockIDType>(Offset);
		BlockIDType LastBlock = GetBlock<BlockIDType>(Offset + Size - 1);// Block containing the last byte
		return (LastBlock.Get() - FirstBlock.Get()) + 1;
	}

	// Returns the offset within the first block covering the byte range to read from
	template<typename BlockIDType> inline int64 GetBlockOffset(int64 Offset)
	{
		return Offset - FMath::DivideAndRoundDown(Offset, BlockIDType::GetSize()) *  BlockIDType::GetSize();
	}

	// Returns the size within the first cache line covering the byte range to read
	template<typename BlockIDType> inline int64 GetBlockSize(int64 Offset, int64 Size)
	{
		int64 OffsetInBlock = GetBlockOffset<BlockIDType>(Offset);
		return FMath::Min((int64)(BlockIDType::BlockSize - OffsetInBlock), Size - Offset);
	}

	virtual IMemoryReadStreamRef ReadData(FGraphEventArray& OutCompletionEvents, int64 Offset, int64 BytesToRead, EAsyncIOPriorityAndFlags Priority) override;
	virtual FGraphEventRef PreloadData(const FFileCachePreloadEntry* PreloadEntries, int32 NumEntries, EAsyncIOPriorityAndFlags Priority) override;

	IMemoryReadStreamRef ReadDataUncached(FGraphEventArray& OutCompletionEvents, int64 Offset, int64 BytesToRead, EAsyncIOPriorityAndFlags Priority);

	void WaitAll() override;

	void Evict(CacheLineID Line);

private:
	struct FPendingRequest
	{
		FGraphEventRef Event;
	};
	struct FLineInfo
	{
		CacheSlotID SlotID;
		FPendingRequest PendingRequest;
	};
	struct FValidLineInfo
	{
		FLineInfo& LineInfo;
		bool bWasValid = false;
	};


	void CheckForSizeRequestComplete();

	FValidLineInfo AcquireSlotAndReadLine(FFileCache& Cache, CacheLineID LineID, EAsyncIOPriorityAndFlags Priority);
	void ReadLine(FFileCache& Cache, CacheSlotID SlotID, CacheLineID LineID, EAsyncIOPriorityAndFlags Priority, const FGraphEventRef& CompletionEvent);

	TMap<int32, FLineInfo> LineInfos;

	int64 BaseOffset;
	int64 FileSize;
	IAsyncReadFileHandle* InnerHandle;
	FGraphEventRef SizeRequestEvent;
};

///////////////

#if !UE_BUILD_SHIPPING

static std::atomic_bool bFileCacheInitialized = false;
static std::atomic_uint GIoStoreCompressionBlockSize = 0; // 0 means no iostore has reported any.
static std::atomic_bool bIoStoreCompressionBlockSizeMultiple = false; // if we get different ones, we by definition can't match.

//
// This exists to log warnings when projects are misconfigured to have FileCache block sizes
// different than iostore compression block sizes. The difficulty is that iostore containers
// are mounted _very early_ - before the cvars are resolved - so we don't know what the final
// FileCache block size will be. However, once in the FileCache constructor, there's no code api
// to get us access to the underlying iostore containers - file iostore containers are above core.
// So we have to have that code call over to us when it's loaded, and we just keep track of what
// compression blocks we've seen.
//
// Once we initialize, we check and log as necessary.
//
// And of course, iostore containers can be mounted _after_ the file cache is initialized,
// so we have to check for that and log immediately in that case.
//
void FileCache_PostIoStoreCompressionBlockSize(uint32 InCompressionBlockSize, FString const& InContainerFilePath)
{
	if (bFileCacheInitialized.load())
	{
		// We can direct check since CacheSlotID is correct (cvars resolved).
		if (InCompressionBlockSize != CacheSlotID::GetSize())
		{
			UE_LOG(LogStreamingFileCache, Warning, TEXT("IoStore container %s has a different block sizes than FileCache (%d vs %" INT64_FMT ")!"), *InContainerFilePath, InCompressionBlockSize, CacheSlotID::GetSize());
			UE_LOGF(LogStreamingFileCache, Warning, "	Check your IoStore compression block size (Project Settings -> 'Package Compression Commandline Options'");
			UE_LOGF(LogStreamingFileCache, Warning, "	and your File Cache block size (fc.BlockSize cvar). They should match!");
		}
		return;
	}

	// otherwise, save off the value.
	uint32 LastBlockSize = GIoStoreCompressionBlockSize.exchange(InCompressionBlockSize);
	if (LastBlockSize && // we had a value
		LastBlockSize != InCompressionBlockSize) // and it was different
	{
		// Mark that we are dealing with more than one compression blocks size.
		bIoStoreCompressionBlockSizeMultiple.store(true);
	}
}

#endif //!UE_BUILD_SHIPPING

FFileCache::FFileCache(int32 NumSlots)
	: EvictFileCacheCommand(TEXT("r.VT.EvictFileCache"), TEXT("Evict all the file caches in the VT system."),
		FConsoleCommandDelegate::CreateRaw(this, &FFileCache::EvictFileCacheFromConsole))
	, SizeInBytes(NumSlots * CacheSlotID::GetSize())
	, NumFreeSlots(NumSlots)
	, MinNumFreeSlots(NumSlots)
{
	LLM_SCOPE(ELLMTag::FileSystem);

#if !UE_BUILD_SHIPPING
	//
	// If we aren't shipping, check and see if we have a mismatch vs the compression
	// block size for any mounted iostore containers.
	//
	uint32 IoStoreCompressionBlockSize = GIoStoreCompressionBlockSize.load();
	if (bIoStoreCompressionBlockSizeMultiple.load())
	{
		UE_LOGF(LogStreamingFileCache, Warning, "IoStore containers have multiple compression block sizes! This means the FileCache block size must be misaligned with at least one!");
		UE_LOGF(LogStreamingFileCache, Warning, "	Check your IoStore compression block size (Project Settings -> 'Package Compression Commandline Options'");
		UE_LOGF(LogStreamingFileCache, Warning, "	and your File Cache block size (fc.BlockSize cvar). They should match!");
	}
	else if (IoStoreCompressionBlockSize && // If we load without any iostore containers (i.e. editor) we don't need to warn.
		IoStoreCompressionBlockSize != CacheSlotID::GetSize())
	{
		UE_LOG(LogStreamingFileCache, Warning, TEXT("IoStore containers have a different block sizes than FileCache (%d vs %" INT64_FMT ")!"), IoStoreCompressionBlockSize, CacheSlotID::GetSize());
		UE_LOGF(LogStreamingFileCache, Warning, "	Check your IoStore compression block size (Project Settings -> 'Package Compression Commandline Options'");
		UE_LOGF(LogStreamingFileCache, Warning, "	and your File Cache block size (fc.BlockSize cvar). They should match!");
	}
#endif // !UE_BUILD_SHIPPING

	Memory = (uint8*)FMemory::Malloc(SizeInBytes);

	SlotInfo.AddUninitialized(NumSlots + 1);
	for (int i = 0; i <= NumSlots; ++i)
	{
		FSlotInfo& Info = SlotInfo[i];
		Info.Handle = nullptr;
		Info.LineID = CacheLineID();
		Info.LockCount = 0;
		Info.NextSlotIndex = i + 1;
		Info.PrevSlotIndex = i - 1;
	}

	// list is circular
	SlotInfo[0].PrevSlotIndex = NumSlots;
	SlotInfo[NumSlots].NextSlotIndex = 0;

#if !UE_BUILD_SHIPPING
	bFileCacheInitialized.store(true);
#endif

	FCoreDelegates::OnBeginFrameRT.AddLambda([this]() { 
		CSV_CUSTOM_STAT(FileCache, NumFreeSlots, NumFreeSlots, ECsvCustomStatOp::Set); 
		CSV_CUSTOM_STAT(FileCache, MinNumFreeSlots, MinNumFreeSlots, ECsvCustomStatOp::Set);
		});
}

CacheSlotID FFileCache::AcquireAndLockSlot(FFileCacheHandle* InHandle, CacheLineID InLineID)
{
	check(NumFreeSlots > 0);
	--NumFreeSlots;
	MinNumFreeSlots = FMath::Min(MinNumFreeSlots, NumFreeSlots);

	const int32 SlotIndex = SlotInfo[0].NextSlotIndex;
	check(SlotIndex != 0);

	FSlotInfo& Info = SlotInfo[SlotIndex];
	check(Info.LockCount == 0); // slot should not be in free list if it's locked
	if (Info.Handle)
	{
		Info.Handle->Evict(Info.LineID);
	}

	Info.LockCount = 1;
	Info.Handle = InHandle;
	Info.LineID = InLineID;
	UnlinkSlot(SlotIndex);

	return CacheSlotID(SlotIndex - 1);
}

bool FFileCache::IsSlotLocked(CacheSlotID InSlotID) const
{
	const int32 SlotIndex = InSlotID.Get() + 1;
	const FSlotInfo& Info = SlotInfo[SlotIndex];
	return Info.LockCount > 0;
}

void FFileCache::LockSlot(CacheSlotID InSlotID)
{
	const int32 SlotIndex = InSlotID.Get() + 1;
	FSlotInfo& Info = SlotInfo[SlotIndex];
	const int32 PrevLockCount = Info.LockCount;
	if (PrevLockCount == 0)
	{
		check(NumFreeSlots > 0);
		--NumFreeSlots;
		MinNumFreeSlots = FMath::Min(MinNumFreeSlots, NumFreeSlots);
		UnlinkSlot(SlotIndex);
	}
	Info.LockCount = PrevLockCount + 1;
}

void FFileCache::UnlockSlot(CacheSlotID InSlotID)
{
	const int32 SlotIndex = InSlotID.Get() + 1;
	const int32 PrevLockCount = SlotInfo[SlotIndex].LockCount;
	check(PrevLockCount > 0);
	if (PrevLockCount == 1)
	{
		// move slot back to the free list when it's unlocked
		LinkSlotTail(SlotIndex);
		++NumFreeSlots;
		check(NumFreeSlots < SlotInfo.Num());
	}
	SlotInfo[SlotIndex].LockCount = PrevLockCount - 1;
}

bool FFileCache::EvictAll(FFileCacheHandle* InFile)
{
	SCOPE_CYCLE_COUNTER(STAT_SFC_EvictAll);

	FScopeLock Lock(&CriticalSection);

	bool bAllOK = true;
	for (int SlotIndex = 1; SlotIndex < SlotInfo.Num(); ++SlotIndex)
	{
		FSlotInfo& Info = SlotInfo[SlotIndex];
		if (Info.Handle && ((Info.Handle == InFile) || InFile == nullptr))
		{
			if (Info.LockCount == 0)
			{
				Info.Handle->Evict(Info.LineID);
				Info.Handle = nullptr;
				Info.LineID = CacheLineID();

				// move evicted slots to the front of list so they'll be re-used more quickly
				UnlinkSlot(SlotIndex);
				LinkSlotHead(SlotIndex);
			}
			else
			{
				bAllOK = false;
			}
		}
	}

	return bAllOK;
}

void FFileCache::FlushCompletedRequests()
{
	while (IAsyncReadRequest* Request = CompletedRequests.Pop())
	{
		Request->WaitCompletion();
		delete Request;
	}
}

FFileCacheHandle::~FFileCacheHandle()
{
	if (SizeRequestEvent)
	{
		FTaskGraphInterface::Get().WaitUntilTaskCompletes(SizeRequestEvent);
		SizeRequestEvent.SafeRelease();
	}

	if (InnerHandle)
	{
		WaitAll();

		const bool result = GetCache().EvictAll(this);
		check(result);

		// Need to ensure any request created by our async handle is destroyed before destroying the handle
		GetCache().FlushCompletedRequests();

		delete InnerHandle;
	}
}

FFileCacheHandle::FFileCacheHandle(IAsyncReadFileHandle* InHandle, int64 InBaseOffset)
	: BaseOffset(InBaseOffset)
	, FileSize(-1)
	, InnerHandle(InHandle)
{
	FGraphEventRef CompletionEvent = FGraphEvent::CreateGraphEvent();
	FAsyncFileCallBack SizeCallbackFunction = [this, CompletionEvent](bool bWasCancelled, IAsyncReadRequest* Request)
	{
		this->FileSize = Request->GetSizeResults();
		check(this->FileSize > 0);

		CompletionEvent->DispatchSubsequents();
		GetCache().PushCompletedRequest(Request);
	};

	SizeRequestEvent = CompletionEvent;
	IAsyncReadRequest* SizeRequest = InHandle->SizeRequest(&SizeCallbackFunction);
	check(SizeRequest);
}

class FMemoryReadStreamAsyncRequest : public IMemoryReadStream
{
public:
	FMemoryReadStreamAsyncRequest(IAsyncReadRequest* InRequest, int64 InSize)
		: Memory(nullptr), Request(InRequest), Size(InSize)
	{
	}

	uint8* GetReadResults()
	{
		if (Request)
		{
			// Event is triggered from read callback, so small window where event is triggered, but request isn't flagged as complete
			// Normally this wait won't be needed
			check(!Memory);
			Request->WaitCompletion();
			Memory = Request->GetReadResults(); // We now own the pointer returned from GetReadResults()
			delete Request; // no longer need to keep request alive
			Request = nullptr;
		}
		return Memory;
	}

	virtual const void* Read(int64& OutSize, int64 InOffset, int64 InSize) override
	{
		const uint8* ResultData = GetReadResults();
		check(InOffset < Size);
		OutSize = FMath::Min(InSize, Size - InOffset);
		return ResultData + InOffset;
	}

	virtual void EnsureReadNonBlocking() override
	{
		if (Request)
		{
			check(!Memory);
			Request->EnsureCompletion();
		}
	}

	virtual int64 GetSize() override
	{
		return Size;
	}

	virtual ~FMemoryReadStreamAsyncRequest()
	{
		uint8* ResultData = GetReadResults();
		if (ResultData)
		{
			FMemory::Free(ResultData);
		}
		check(!Request);
	}

	uint8* Memory;
	IAsyncReadRequest* Request;
	int64 Size;
};

IMemoryReadStreamRef FFileCacheHandle::ReadDataUncached(FGraphEventArray& OutCompletionEvents, int64 Offset, int64 BytesToRead, EAsyncIOPriorityAndFlags Priority)
{
	FGraphEventRef CompletionEvent = FGraphEvent::CreateGraphEvent();

	FAsyncFileCallBack ReadCallbackFunction = [CompletionEvent](bool bWasCancelled, IAsyncReadRequest* Request)
	{
		CompletionEvent->DispatchSubsequents();
	};

	OutCompletionEvents.Add(CompletionEvent);
	IAsyncReadRequest* AsyncRequest = InnerHandle->ReadRequest(Offset, BytesToRead, Priority, &ReadCallbackFunction);
	return new FMemoryReadStreamAsyncRequest(AsyncRequest, BytesToRead);
}

class FMemoryReadStreamCache : public IMemoryReadStream
{
public:
	virtual const void* Read(int64& OutSize, int64 InOffset, int64 InSize) override
	{
		FFileCache& Cache = GetCache();

		const int64 Offset = InitialSlotOffset + InOffset;
		const int64 BlockSize = CacheSlotID::GetSize();
		const int32 SlotIndex = (int32)FMath::DivideAndRoundDown(Offset, BlockSize);
		const int32 OffsetInSlot = (int32)(Offset - SlotIndex * BlockSize);
		check(SlotIndex >= 0 && SlotIndex < NumCacheSlots);
		const void* SlotMemory = Cache.GetSlotMemory(CacheSlots[SlotIndex]);

		OutSize = FMath::Min(InSize, BlockSize - OffsetInSlot);
		return (uint8*)SlotMemory + OffsetInSlot;
	}

	virtual int64 GetSize() override
	{
		return Size;
	}

	virtual ~FMemoryReadStreamCache()
	{
		FFileCache& Cache = GetCache();
		FScopeLock CacheLock(&Cache.CriticalSection);
		for (int i = 0; i < NumCacheSlots; ++i)
		{
			const CacheSlotID& SlotID = CacheSlots[i];
			check(SlotID.IsValid());
			Cache.UnlockSlot(SlotID);
		}
	}

	void operator delete(void* InMem)
	{
		FMemory::Free(InMem);
	}

	int64 InitialSlotOffset;
	int64 Size;
	int32 NumCacheSlots;
	CacheSlotID CacheSlots[1]; // variable length, sized by NumCacheSlots
};

void FFileCacheHandle::CheckForSizeRequestComplete()
{
	if (SizeRequestEvent && SizeRequestEvent->IsComplete())
	{
		SizeRequestEvent.SafeRelease();

		check(FileSize > 0);

		// LineInfos key is int32
		const int64 TotalNumSlots = FMath::DivideAndRoundUp(FileSize, (int64)GFileCacheBlockSizeKB * 1024);
		check(TotalNumSlots < MAX_int32);
	}
}

void FFileCacheHandle::ReadLine(FFileCache& Cache, CacheSlotID SlotID, CacheLineID LineID, EAsyncIOPriorityAndFlags Priority, const FGraphEventRef& CompletionEvent)
{
	check(FileSize >= 0);
	const int64 LineSizeInFile = LineID.GetSize(FileSize);
	const int64 LineOffsetInFile = LineID.GetOffset();
	uint8* CacheSlotMemory = Cache.GetSlotMemory(SlotID);

	// callback triggered when async read operation is complete, used to signal task graph event
	FAsyncFileCallBack ReadCallbackFunction = [CompletionEvent](bool bWasCancelled, IAsyncReadRequest* Request)
	{
		CompletionEvent->DispatchSubsequents();
		GetCache().PushCompletedRequest(Request);
	};

	InnerHandle->ReadRequest(LineOffsetInFile, LineSizeInFile, Priority, &ReadCallbackFunction, CacheSlotMemory);
}

FFileCacheHandle::FValidLineInfo FFileCacheHandle::AcquireSlotAndReadLine(FFileCache& Cache, CacheLineID LineID, EAsyncIOPriorityAndFlags Priority)
{
	SCOPED_LOADTIMER(FFileCacheHandle_AcquireSlotAndReadLine);

	// Check if there's a valid line already and return that
	{
		FLineInfo& LineInfo = LineInfos.FindOrAdd(LineID.Get());
		if (LineInfo.SlotID.IsValid())
		{
			return { LineInfo, true };
		}
	}

	// no valid slot for this line, grab a new slot from cache and start a read request
	CacheSlotID SlotID = Cache.AcquireAndLockSlot(this, LineID);

	FLineInfo& LineInfo = LineInfos.FindOrAdd(LineID.Get());
	LineInfo.SlotID = SlotID;
	if (LineInfo.PendingRequest.Event)
	{
		// previous async request/event (if any) should be completed, if this is back in the free list
		check(LineInfo.PendingRequest.Event->IsComplete());
	}

	FGraphEventRef CompletionEvent = FGraphEvent::CreateGraphEvent();
	LineInfo.PendingRequest.Event = CompletionEvent;
	if (FileSize >= 0)
	{
		// If FileSize >= 0, that means the async file size request has completed, we can perform the read immediately
		ReadLine(Cache, SlotID, LineID, Priority, CompletionEvent);
	}
	else
	{
		// Here we don't know the FileSize yet, so we schedule an async task to kick the read once the size request has completed
		// It's important to know the size of the file before performing the read, to ensure that we don't read past end-of-file
		FFunctionGraphTask::CreateAndDispatchWhenReady([this, SlotID, LineID, Priority, CompletionEvent]
		{
			this->ReadLine(GetCache(), SlotID, LineID, Priority, CompletionEvent);
		},
			TStatId(), SizeRequestEvent);
	}

	return { LineInfo, false };
}

IMemoryReadStreamRef FFileCacheHandle::ReadData(FGraphEventArray& OutCompletionEvents, int64 InOffset, int64 BytesToRead, EAsyncIOPriorityAndFlags Priority)
{
	SCOPE_CYCLE_COUNTER(STAT_SFC_ReadData);
	SCOPED_LOADTIMER(FFileCacheHandle_ReadData);

	const int64 Offset = BaseOffset + InOffset;
	const CacheLineID StartLine = GetBlock<CacheLineID>(Offset);
	const CacheLineID EndLine = GetBlock<CacheLineID>(Offset + BytesToRead - 1);
	check(EndLine.Get() + 1 < MAX_int32);

	const int32 NumSlotsNeeded = EndLine.Get() + 1 - StartLine.Get();

	FFileCache& Cache = GetCache();

	FScopeLock CacheLock(&Cache.CriticalSection);

	CheckForSizeRequestComplete();

	if (NumSlotsNeeded > Cache.NumFreeSlots)
	{
		// not enough free slots in the cache to service this request
		CacheLock.Unlock();

		UE_LOG(LogStreamingFileCache, Verbose, TEXT("ReadData(%" INT64_FMT ", %" INT64_FMT ") is skipping cache, cache is full"), Offset, BytesToRead);
		return ReadDataUncached(OutCompletionEvents, Offset, BytesToRead, Priority);
	}

	const int32 NumCacheSlots = EndLine.Get() + 1 - StartLine.Get();
	check(NumCacheSlots > 0);
	const uint32 AllocSize = sizeof(FMemoryReadStreamCache) + sizeof(CacheSlotID) * (NumCacheSlots - 1);
	void* ResultMemory = FMemory::Malloc(AllocSize, alignof(FMemoryReadStreamCache));
	FMemoryReadStreamCache* Result = new(ResultMemory) FMemoryReadStreamCache();
	Result->NumCacheSlots = NumCacheSlots;
	Result->InitialSlotOffset = GetBlockOffset<CacheLineID>(Offset);
	Result->Size = BytesToRead;

	bool bHasPendingSlot = false;
	for (CacheLineID LineID = StartLine; LineID.Get() <= EndLine.Get(); ++LineID)
	{
		FValidLineInfo ValidLineInfo = AcquireSlotAndReadLine(Cache, LineID, Priority);

		FLineInfo& LineInfo = ValidLineInfo.LineInfo;
		if (ValidLineInfo.bWasValid)
		{
			// if it was already valid lock the slot for reading
			Cache.LockSlot(LineInfo.SlotID);
		}

		check(LineInfo.SlotID.IsValid());
		Result->CacheSlots[LineID.Get() - StartLine.Get()] = LineInfo.SlotID;

		if (LineInfo.PendingRequest.Event && !LineInfo.PendingRequest.Event->IsComplete())
		{
			// this line has a pending async request to read data
			// will need to wait for this request to complete before data is valid
			OutCompletionEvents.Add(LineInfo.PendingRequest.Event);
			bHasPendingSlot = true;
		}
		else
		{
			LineInfo.PendingRequest.Event.SafeRelease();
		}
	}

	return Result;
}

struct FFileCachePreloadTask
{
	explicit FFileCachePreloadTask(TArray<CacheSlotID>&& InLockedSlots) : LockedSlots(InLockedSlots) {}
	TArray<CacheSlotID> LockedSlots;

	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
	{
		FFileCache& Cache = GetCache();
		FScopeLock CacheLock(&Cache.CriticalSection);
		for (int i = 0; i < LockedSlots.Num(); ++i)
		{
			const CacheSlotID& SlotID = LockedSlots[i];
			check(SlotID.IsValid());
			Cache.UnlockSlot(SlotID);
		}
	}

	FORCEINLINE static ESubsequentsMode::Type GetSubsequentsMode() { return ESubsequentsMode::TrackSubsequents; }
	FORCEINLINE ENamedThreads::Type GetDesiredThread() { return ENamedThreads::AnyNormalThreadNormalTask; }
	FORCEINLINE TStatId GetStatId() const { return TStatId(); }
};

FGraphEventRef FFileCacheHandle::PreloadData(const FFileCachePreloadEntry* PreloadEntries, int32 NumEntries, EAsyncIOPriorityAndFlags Priority)
{
	SCOPED_LOADTIMER(FFileCacheHandle_PreloadData);

	check(NumEntries > 0);

	FFileCache& Cache = GetCache();

	FScopeLock CacheLock(&Cache.CriticalSection);

	CheckForSizeRequestComplete();

	FGraphEventArray CompletionEvents;
	TArray<CacheSlotID> LockedSlots;
	LockedSlots.Empty(NumEntries);

	CacheLineID CurrentLine(0);
	int64 PrevOffset = -1;
	for (int32 EntryIndex = 0; EntryIndex < NumEntries && Cache.NumFreeSlots > 0; ++EntryIndex)
	{
		const FFileCachePreloadEntry& Entry = PreloadEntries[EntryIndex];
		const int64 EntryOffset = BaseOffset + Entry.Offset;
		const CacheLineID StartLine = GetBlock<CacheLineID>(EntryOffset);
		const CacheLineID EndLine = GetBlock<CacheLineID>(EntryOffset + Entry.Size - 1);

		check(EndLine.Get() + 1 < MAX_int32);
		checkf(Entry.Offset > PrevOffset, TEXT("Preload entries must be sorted by Offset [%lld, %lld), %lld"),
			Entry.Offset, Entry.Offset + Entry.Size, PrevOffset);
		PrevOffset = Entry.Offset;

		CurrentLine = CacheLineID(FMath::Max(CurrentLine.Get(), StartLine.Get()));
		while (CurrentLine.Get() <= EndLine.Get() && Cache.NumFreeSlots > 0)
		{
			FValidLineInfo ValidLineInfo = AcquireSlotAndReadLine(Cache, CurrentLine, Priority);

			FLineInfo& LineInfo = ValidLineInfo.LineInfo;
			
			// line info was not valid and just created, grab a new slot from cache and start a read request
			if (!ValidLineInfo.bWasValid)
			{
				LockedSlots.Add(LineInfo.SlotID);
			}

			if (LineInfo.PendingRequest.Event && !LineInfo.PendingRequest.Event->IsComplete())
			{
				// this line has a pending async request to read data
				// will need to wait for this request to complete before data is valid
				CompletionEvents.Add(LineInfo.PendingRequest.Event);
			}
			else
			{
				LineInfo.PendingRequest.Event.SafeRelease();
			}

			++CurrentLine;
		}
	}

	FGraphEventRef CompletionEvent;
	if (CompletionEvents.Num() > 0)
	{
		CompletionEvent = TGraphTask<FFileCachePreloadTask>::CreateTask(&CompletionEvents).ConstructAndDispatchWhenReady(MoveTemp(LockedSlots));
	}
	else if (LockedSlots.Num() > 0)
	{
		// Unusual case, we locked some slots, but the reads completed immediately, so we don't need to keep the slots locked
		for (const CacheSlotID& SlotID : LockedSlots)
		{
			Cache.UnlockSlot(SlotID);
		}
	}

	return CompletionEvent;
}

void FFileCacheHandle::Evict(CacheLineID LineID)
{
	FLineInfo* LineInfo = LineInfos.Find(LineID.Get());
	if (LineInfo != nullptr)
	{ 
		if (LineInfo->PendingRequest.Event)
		{
			check(LineInfo->PendingRequest.Event->IsComplete());
			LineInfo->PendingRequest.Event.SafeRelease();
		}

		LineInfos.Remove(LineID.Get());
	}
}

void FFileCacheHandle::WaitAll()
{
	for (TPair<int32, FLineInfo>& Pair : LineInfos)
	{
		FLineInfo& LineInfo = Pair.Value;
		if (LineInfo.PendingRequest.Event)
		{
			check(LineInfo.PendingRequest.Event->IsComplete());
			LineInfo.PendingRequest.Event.SafeRelease();
		}
	}
}

void IFileCacheHandle::EvictAll()
{
	GetCache().EvictAll();
}

IFileCacheHandle* IFileCacheHandle::CreateFileCacheHandle(const TCHAR* InFileName, int64 InBaseOffset)
{
	SCOPE_CYCLE_COUNTER(STAT_SFC_CreateHandle);

	IAsyncReadFileHandle* FileHandle = FPlatformFileManager::Get().GetPlatformFile().OpenAsyncRead(InFileName);
	if (!FileHandle)
	{
		return nullptr;
	}

	return new FFileCacheHandle(FileHandle, InBaseOffset);
}

IFileCacheHandle* IFileCacheHandle::CreateFileCacheHandle(IAsyncReadFileHandle* FileHandle, int64 InBaseOffset)
{
	SCOPE_CYCLE_COUNTER(STAT_SFC_CreateHandle);

	if (FileHandle != nullptr)
	{
		return new FFileCacheHandle(FileHandle, InBaseOffset);
	}
	else
	{
		return nullptr;
	}
}

int64 IFileCacheHandle::GetFileCacheSize()
{
	return GetCache().SizeInBytes;
}
