/*
Copyright 2019 PureDev Software Limited

Permission to use, copy, modify, and/or distribute this software for any
purpose with or without fee is hereby granted, provided that the above
copyright notice and this permission notice appear in all copies.

THE SOFTWARE IS PROVIDED "AS IS" AND THE AUTHOR DISCLAIMS ALL WARRANTIES
WITH REGARD TO THIS SOFTWARE INCLUDING ALL IMPLIED WARRANTIES OF
MERCHANTABILITY AND FITNESS. IN NO EVENT SHALL THE AUTHOR BE LIABLE FOR
ANY SPECIAL, DIRECT, INDIRECT, OR CONSEQUENTIAL DAMAGES OR ANY DAMAGES
WHATSOEVER RESULTING FROM LOSS OF USE, DATA OR PROFITS, WHETHER IN AN
ACTION OF CONTRACT, NEGLIGENCE OR OTHER TORTIOUS ACTION, ARISING OUT OF
OR IN CONNECTION WITH THE USE OR PERFORMANCE OF THIS SOFTWARE.
*/

#if defined(__UNREAL__)
	#include "FramePro/FramePro.h"
#else
	#include "FramePro.h"
#endif

#include "AutoRTFM.h"

//------------------------------------------------------------------------
#if FRAMEPRO_ENABLED

//------------------------------------------------------------------------
//
// FrameProLib.h
//

//------------------------------------------------------------------------
#include <stdio.h>
#include <stdlib.h>
#include <string.h>
#include <wchar.h>
#include <limits.h>
#include <new>

//------------------------------------------------------------------------
namespace FramePro
{
	//------------------------------------------------------------------------
	enum { g_FrameProLibVersion = 17 };

	//------------------------------------------------------------------------
	void DebugWrite(const char* p_str, ...);

	//------------------------------------------------------------------------
	class CheckedAllocator : public Allocator
	{
	public:
		CheckedAllocator()
		:	mp_Allocator(NULL)
		{
		}

		bool HasAllocator() const
		{
			return mp_Allocator != NULL;
		}

		void SetAllocator(Allocator* p_allocator)
		{
			mp_Allocator = p_allocator;
		}

		void* Alloc(size_t size)
		{
			if (!mp_Allocator)
				return NULL;

			void* p = mp_Allocator->Alloc(size);
			if (!p)
				HandleOOM();

			return p;
		}

		void Free(void* p)
		{
			if(mp_Allocator)
				mp_Allocator->Free(p);
		}

	private:
		void HandleOOM()
		{
			DebugWrite("*******************************\n");
			DebugWrite("FramePro: ERROR. Out of memory!\n");
			DebugWrite("*******************************\n");

			DebugBreak();
		}

		//------------------------------------------------------------------------
		// data
	private:
		Allocator* mp_Allocator;
	};

	//------------------------------------------------------------------------
	template<class T>
	inline T* New(Allocator* p_allocator)
	{
		T* p = (T*)p_allocator->Alloc(sizeof(T));
		if (p)
			new (p)T();
		return p;
	}

	//------------------------------------------------------------------------
	template<class T, typename Targ1>
	inline T* New(Allocator* p_allocator, Targ1 arg1)
	{
		T* p = (T*)p_allocator->Alloc(sizeof(T));
		if (p)
			new (p)T(arg1);
		return p;
	}

	//------------------------------------------------------------------------
	template<class T, typename Targ1, typename Targ2, typename TArg3>
	inline T* New(Allocator* p_allocator, Targ1 arg1, Targ2 arg2, TArg3 arg3)
	{
		T* p = (T*)p_allocator->Alloc(sizeof(T));
		if (p)
			new (p)T(arg1, arg2, arg3);
		return p;
	}

	//------------------------------------------------------------------------
	template<typename T>
	inline void Delete(Allocator* p_allocator, T* p)
	{
		p->~T();
		p_allocator->Free(p);
	}

	//------------------------------------------------------------------------
	namespace StringLiteralType
	{
		enum Enum
		{
			NameAndSourceInfo = 0,
			NameAndSourceInfoW,
			SourceInfo,
			GeneralString,
			StringLiteralTimerName,
			GeneralStringW,
		};
	}

	//------------------------------------------------------------------------
	class FrameProTLS;

	//------------------------------------------------------------------------
	extern FRAMEPRO_NO_INLINE FrameProTLS* CreateFrameProTLS();
	extern FRAMEPRO_NO_INLINE void DestroyFrameProTLS(FrameProTLS* p_framepro_tls);

	#if FRAMEPRO_USE_TLS_SLOTS
		extern FRAMEPRO_NO_INLINE uint GetFrameProTLSSlot();
	#else
		inline uint GetFrameProTLSSlot() { return 0; } // help the compiler optimise this away if we are not using tls TLS slots
	#endif

	//------------------------------------------------------------------------
	FRAMEPRO_FORCE_INLINE FrameProTLS* GetFrameProTLS()
	{
		FrameProTLS* p_framepro_tls = (FrameProTLS*)Platform::GetTLSValue(GetFrameProTLSSlot());
		return p_framepro_tls ? p_framepro_tls : CreateFrameProTLS();
	}

	//------------------------------------------------------------------------
	FRAMEPRO_FORCE_INLINE FrameProTLS* TryGetFrameProTLS()
	{
		return (FrameProTLS*)Platform::GetTLSValue(GetFrameProTLSSlot());
	}

	//------------------------------------------------------------------------
	FRAMEPRO_FORCE_INLINE void ClearFrameProTLS()
	{
		Platform::SetTLSValue(GetFrameProTLSSlot(), NULL);
	}

	//------------------------------------------------------------------------
	inline bool IsPow2(int value)
	{
		return (value & (value-1)) == 0;
	}

	//------------------------------------------------------------------------
	inline int AlignUpPow2(int value, int alignment)
	{
		FRAMEPRO_ASSERT(IsPow2(alignment));		// non-pow2 value passed to align function
		int mask = alignment - 1;
		return (value + mask) & ~mask;
	}

	//------------------------------------------------------------------------
	template<typename T>
	inline T FramePro_Min(T a, T b)
	{
		return a < b ? a : b;
	}

	//------------------------------------------------------------------------
	template<typename T>
	inline T FramePro_Max(T a, T b)
	{
		return a > b ? a : b;
	}

	//------------------------------------------------------------------------
	template<typename T>
	inline void Swap(T& a, T& b)
	{
		T temp = a;
		a = b;
		b = temp;
	}

	//------------------------------------------------------------------------
	void SPrintf(char* p_buffer, size_t const buffer_size, const char* p_format, ...);
}


//------------------------------------------------------------------------
//
// EventTraceWin32.h
//

//------------------------------------------------------------------------
#if FRAMEPRO_PLATFORM_WIN

//------------------------------------------------------------------------
#if FRAMEPRO_ENABLE_CONTEXT_SWITCH_TRACKING

//------------------------------------------------------------------------
namespace FramePro
{
	//------------------------------------------------------------------------
	class Allocator;
	class EventTraceWin32Imp;
	class DynamicString;

	//------------------------------------------------------------------------
	typedef void (*ContextSwitchCallback)(const ContextSwitch& context_switch, void* p_param);

	//------------------------------------------------------------------------
	class EventTraceWin32
	{
	public:
		EventTraceWin32(Allocator* p_allocator);

		~EventTraceWin32();

		bool Start(ContextSwitchCallback p_context_switch_callback, void* p_context_switch_callback_param, DynamicString& error);

		void Stop();

		void Flush();

		static void* Create(Allocator* p_allocator);

		static void Destroy(void* p_context_switch_recorder, Allocator* p_allocator);

		static bool Start(void* p_context_switch_recorder, Platform::ContextSwitchCallbackFunction p_callback, void* p_context, DynamicString& error);

		static void Stop(void* p_context_switch_recorder);
	
		static void Flush(void* p_context_switch_recorder);

		//------------------------------------------------------------------------
		// data
	private:
		EventTraceWin32Imp* mp_Imp;
		Allocator* mp_Allocator;
	};
}

//------------------------------------------------------------------------
#endif		// #if FRAMEPRO_ENABLE_CONTEXT_SWITCH_TRACKING

//------------------------------------------------------------------------
#endif		// #if FRAMEPRO_PLATFORM_WIN


//------------------------------------------------------------------------
//
// CriticalSection.h
//

//------------------------------------------------------------------------
namespace FramePro
{
	//------------------------------------------------------------------------
	class CriticalSection
	{
	public:
		//------------------------------------------------------------------------
		CriticalSection()
		{
			Platform::CreateLock(m_OSLockMem, sizeof(m_OSLockMem));

			#if FRAMEPRO_DEBUG
				m_Locked = false;
				m_LockedOnThread = 0xffffffffffffffffULL;
			#endif
		}

		//------------------------------------------------------------------------
		~CriticalSection()
		{
			Platform::DestroyLock(m_OSLockMem);
		}

		//------------------------------------------------------------------------
		void Enter()
		{
			FRAMEPRO_ASSERT((uint64)Platform::GetCurrentThreadId() != m_LockedOnThread);

			Platform::TakeLock(m_OSLockMem);

			#if FRAMEPRO_DEBUG
				m_Locked = true;
				m_LockedOnThread = Platform::GetCurrentThreadId();
			#endif
		}

		//------------------------------------------------------------------------
		void Leave()
		{
			#if FRAMEPRO_DEBUG
				m_Locked = false;
				m_LockedOnThread = 0xffffffffffffffffULL;
			#endif

			Platform::ReleaseLock(m_OSLockMem);
		}

		//------------------------------------------------------------------------
		#if FRAMEPRO_DEBUG
			bool Locked() const		// only safe to use in an assert to check that it IS locked
			{
				return m_Locked;
			}
		#endif

		//------------------------------------------------------------------------
		// data
	private:
		static const int m_OSLockMaxSize = 40;

		char m_OSLockMem[m_OSLockMaxSize];

#if FRAMEPRO_DEBUG
		RelaxedAtomic<bool> m_Locked;
		uint64 m_LockedOnThread;
#endif
	} FRAMEPRO_ALIGN_STRUCT(16);

	//------------------------------------------------------------------------
	class CriticalSectionScope
	{
	public:
		CriticalSectionScope(CriticalSection& in_cs) : cs(in_cs) { cs.Enter(); }
		~CriticalSectionScope() { cs.Leave(); }
	private:
		CriticalSectionScope(const CriticalSectionScope&);
		CriticalSectionScope& operator=(const CriticalSectionScope&);
		CriticalSection& cs;
	};
}


//------------------------------------------------------------------------
//
// HashMap.h
//

//------------------------------------------------------------------------
#include <new>

//------------------------------------------------------------------------
#define FRAMEPRO_PROFILE_HASHMAP 0

//------------------------------------------------------------------------
namespace FramePro
{
	//------------------------------------------------------------------------
	typedef unsigned int uint;
	typedef char byte;

	//------------------------------------------------------------------------
	template<typename TKey, typename TValue>
	class HashMap
	{
	public:
		//------------------------------------------------------------------------
		struct Pair
		{
			TKey m_Key;
			TValue m_Value;
		};

		//------------------------------------------------------------------------
		// The default capacity of the set. The capacity is the number
		// of elements that the set is expected to hold. The set will resized
		// when the item count is greater than the capacity;
		HashMap(Allocator* p_allocator)
		:	m_Capacity(0),
			mp_Table(NULL),
			m_Count(0),
			mp_ItemPool(NULL),
			mp_FreePair(NULL),
			mp_Allocator(p_allocator)
			#if FRAMEPRO_PROFILE_HASHMAP
				,m_IterAcc(0)
				,m_IterCount(0)
			#endif
		{
			AllocTable(GetNextPow2((256 * m_DefaultCapacity) / m_Margin));
		}

		//------------------------------------------------------------------------
		~HashMap()
		{
			Clear();

			mp_Allocator->Free(mp_Table);
			FreePools();
		}

		//------------------------------------------------------------------------
		void Clear()
		{
			RemoveAll();
		}

		//------------------------------------------------------------------------
		void RemoveAll()
		{
			for(int i=0; i<m_Capacity; ++i)
			{
				Pair* p_pair = mp_Table[i];
				if(p_pair)
				{
					FreePair(p_pair);
					mp_Table[i] = NULL;
				}
			}
			m_Count = 0;
		}

		//------------------------------------------------------------------------
		// Add a value to this set.
		// If this set already contains the value does nothing.
		bool Add(const TKey& key, const TValue& value)
		{
			int index = GetItemIndex(key);

			if(IsItemInUse(index))
			{
				mp_Table[index]->m_Value = value;
			}
			else
			{
				if(m_Capacity == 0 || m_Count == (m_Margin * m_Capacity) / 256)
				{
					if (!Resize(2 * m_Capacity))
						return false;

					index = GetItemIndex(key);
				}

				// make a copy of the value
				Pair* p_pair = AllocPair();
				if (!p_pair)
					return false;

				p_pair->m_Key = key;
				p_pair->m_Value = value;

				// add to table
				mp_Table[index] = p_pair;

				++m_Count;
			}

			return true;
		}

		//------------------------------------------------------------------------
		// if this set contains the value set value to the existing value and
		// return true, otherwise set to the default value and return false.
		bool TryGetValue(const TKey& key, TValue& value) const
		{
			if(!mp_Table)
				return false;

			const int index = GetItemIndex(key);
			if(IsItemInUse(index))
			{
				value = mp_Table[index]->m_Value;
				return true;
			}
			else
			{
				return false;
			}
		}

		//------------------------------------------------------------------------
		int GetCount() const
		{
			return m_Count;
		}

		//------------------------------------------------------------------------
		bool Resize(int new_capacity)
		{
			new_capacity = GetNextPow2(new_capacity);

			// keep a copy of the old table
			Pair** const p_old_table = mp_Table;
			const int old_capacity = m_Capacity;

			// allocate the new table
			if (!AllocTable(new_capacity))
				return false;

			// copy the values from the old to the new table
			Pair** p_old_pair = p_old_table;
			for(int i=0; i<old_capacity; ++i, ++p_old_pair)
			{
				Pair* p_pair = *p_old_pair;
				if(p_pair)
				{
					const int index = GetItemIndex(p_pair->m_Key);
					mp_Table[index] = p_pair;
				}
			}

			mp_Allocator->Free(p_old_table);

			return true;
		}

		//------------------------------------------------------------------------
		size_t GetMemorySize() const
		{
			size_t table_memory = m_Capacity * sizeof(Pair*);

			size_t item_memory = 0;
			byte* p_pool = mp_ItemPool;
			while(p_pool)
			{
				p_pool = *(byte**)p_pool;
				item_memory += m_ItemBlockSize;
			}

			return table_memory + item_memory;
		}

	private:
		//------------------------------------------------------------------------
		static int GetNextPow2(int value)
		{
			int p = 2;
			while(p < value)
				p *= 2;
			return p;
		}

		//------------------------------------------------------------------------
		bool AllocTable(const int capacity)
		{
			FRAMEPRO_ASSERT(capacity < m_MaxCapacity);
			m_Capacity = capacity;

			// allocate a block of memory for the table
			if(capacity > 0)
			{
				const int size = capacity * sizeof(Pair*);
				mp_Table = (Pair**)mp_Allocator->Alloc(size);
				if (!mp_Table)
					return false;

				memset(mp_Table, 0, size);
			}

			return true;
		}

		//------------------------------------------------------------------------
		bool IsItemInUse(const int index) const
		{
			return mp_Table[index] != NULL;
		}

		//------------------------------------------------------------------------
		int GetItemIndex(const TKey& key) const
		{
			FRAMEPRO_ASSERT(mp_Table);
			const uint hash = key.GetHashCode();
			int srch_index = hash & (m_Capacity-1);
			while(IsItemInUse(srch_index) && !(mp_Table[srch_index]->m_Key == key))
			{
				srch_index = (srch_index + 1) & (m_Capacity-1);
				#if FRAMEPRO_PROFILE_HASHMAP
					++m_IterAcc;
				#endif
			}

			#if FRAMEPRO_PROFILE_HASHMAP
				++m_IterCount;
				double average = m_IterAcc / (double)m_IterCount;
				if(average > 2.0)
				{
					static int64 last_write_time = 0;
					int64 now = 0;
					FRAMEPRO_GET_CLOCK_COUNT(now);
					static int64 timer_freq = Platform::GetTimerFrequency();
					if(now - last_write_time > timer_freq)
					{
						last_write_time = now;
						char temp[64];
						SPrintf(temp, sizeof(temp), "WARNING: HashMap average: %f\n", (float)average);
						Platform::DebugWrite(temp);
					}
				}
			#endif

			return srch_index;
		}

		//------------------------------------------------------------------------
		static bool InRange(
			const int index,
			const int start_index,
			const int end_index)
		{
			return (start_index <= end_index) ?
				index >= start_index && index <= end_index :
				index >= start_index || index <= end_index;
		}

		//------------------------------------------------------------------------
		void FreePools()
		{
			byte* p_pool = mp_ItemPool;
			while(p_pool)
			{
				byte* p_next_pool = *(byte**)p_pool;
				mp_Allocator->Free(p_pool);
				p_pool = p_next_pool;
			}
			mp_ItemPool = NULL;
			mp_FreePair = NULL;
		}

		//------------------------------------------------------------------------
		Pair* AllocPair()
		{
			if(!mp_FreePair)
			{
				// allocate a new pool and link to pool list
				byte* p_new_pool = (byte*)mp_Allocator->Alloc(m_ItemBlockSize);
				if (!p_new_pool)
					return NULL;

				*(byte**)p_new_pool = mp_ItemPool;
				mp_ItemPool = p_new_pool;

				// link all items onto free list
				mp_FreePair = p_new_pool + sizeof(Pair);
				byte* p = (byte*)mp_FreePair;
				int item_count = m_ItemBlockSize / sizeof(Pair) - 2;	// subtract 2 for pool pointer and last item
				FRAMEPRO_ASSERT(item_count);
				for(int i=0; i<item_count; ++i, p+=sizeof(Pair))
				{
					*(byte**)p = p + sizeof(Pair);
				}
				*(byte**)p = NULL;
			}

			// take item off free list
			Pair* p_pair = (Pair*)mp_FreePair;
			mp_FreePair = *(byte**)mp_FreePair;

			// construct the pair
			new (p_pair)Pair;

			return p_pair;
		}

		//------------------------------------------------------------------------
		void FreePair(Pair* p_pair)
		{
			p_pair->~Pair();

			*(byte**)p_pair = mp_FreePair;
			mp_FreePair = (byte*)p_pair;
		}

		//------------------------------------------------------------------------
		// data
	private:
		enum { m_DefaultCapacity = 32 };
		enum { m_InvalidIndex = -1 };
		enum { m_MaxCapacity = 0x7fffffff };
		enum { m_Margin = (30 * 256) / 100 };
		enum { m_ItemBlockSize = 4096 };

		int m_Capacity;			// the current capacity of this set, will always be >= m_Margin*m_Count/256
		Pair** mp_Table;		// NULL for a set with capacity 0
		int m_Count;			// the current number of items in this set, will always be <= m_Margin*m_Count/256

		byte* mp_ItemPool;
		byte* mp_FreePair;

		Allocator* mp_Allocator;

		#if FRAMEPRO_PROFILE_HASHMAP
			mutable int64 m_IterAcc;
			mutable int64 m_IterCount;
		#endif
	};
}


//------------------------------------------------------------------------
//
// IncrementingBlockAllocator.h
//

//------------------------------------------------------------------------
namespace FramePro
{
	//------------------------------------------------------------------------
	class Allocator;

	//------------------------------------------------------------------------
	class IncrementingBlockAllocator
	{
		static const int m_BlockSize = 4096;
		static const int m_MemoryBlockSize = m_BlockSize - sizeof(struct Block*);

		struct Block
		{
			Block* mp_Next;
			char m_Memory[m_MemoryBlockSize];
		};
		static_assert(sizeof(Block) == m_BlockSize, "Block size incorrect");

	public:
		IncrementingBlockAllocator(Allocator* p_allocator);

		~IncrementingBlockAllocator();

		void Clear();

		void* Alloc(size_t size);

		size_t GetMemorySize() const { return m_MemorySize; }

	private:
		bool AllocateBlock();

		//------------------------------------------------------------------------
		// data
	private:
		Allocator* mp_Allocator;

		Block* mp_BlockList;

		size_t m_CurrentBlockSize;

		size_t m_MemorySize;
	};
}


//------------------------------------------------------------------------
//
// FrameProString.h
//

//------------------------------------------------------------------------
#include <string.h>
#include <wchar.h>

//------------------------------------------------------------------------
namespace FramePro
{
	//-------------------------------------------------------------
	// from http://murmurhash.googlepages.com/MurmurHash2.cpp
	inline unsigned int MurmurHash2(const void * key, int len, unsigned int seed)
	{
		// 'm' and 'r' are mixing constants generated offline.
		// They're not really 'magic', they just happen to work well.

		const unsigned int m = 0x5bd1e995;
		const int r = 24;

		// Initialize the hash to a 'random' value

		unsigned int h = seed ^ len;

		// Mix 4 bytes at a time into the hash

		const unsigned char * data = (const unsigned char *)key;

		while(len >= 4)
		{
			unsigned int k = *(unsigned int *)data;

			k *= m; 
			k ^= k >> r; 
			k *= m; 
				
			h *= m; 
			h ^= k;

			data += 4;
			len -= 4;
		}
			
		// Handle the last few bytes of the input array

		switch(len)
		{
		case 3: h ^= data[2] << 16;
		case 2: h ^= data[1] << 8;
		case 1: h ^= data[0];
				h *= m;
		};

		// Do a few final mixes of the hash to ensure the last few
		// bytes are well-incorporated.

		h ^= h >> 13;
		h *= m;
		h ^= h >> 15;

		return h;
	} 

	//-------------------------------------------------------------
	FRAMEPRO_FORCE_INLINE unsigned int MurmurHash2(const char* p_str)
	{
		const unsigned int prime = 0x1000193;
		return MurmurHash2(p_str, (int)strlen(p_str), prime);
	}

	//-------------------------------------------------------------
	FRAMEPRO_FORCE_INLINE unsigned int MurmurHash2(const wchar_t* p_str)
	{
		const unsigned int prime = 0x1000193;
		return MurmurHash2(p_str, (int)wcslen(p_str) * sizeof(wchar_t), prime);
	}

	//------------------------------------------------------------------------
	inline void StringCopy(char* p_dest, size_t dest_size, const char* p_source)
	{
		size_t length = strlen(p_source) + 1;
		FRAMEPRO_ASSERT(length <= dest_size);
		FRAMEPRO_UNREFERENCED(dest_size);
		memcpy(p_dest, p_source, length);
	}

	//------------------------------------------------------------------------
	inline void StringCopy(char* p_dest, size_t dest_size, const char* p_source, size_t source_length)
	{
		size_t length = strlen(p_source) + 1;
		FRAMEPRO_ASSERT(length <= dest_size && source_length < length);
		FRAMEPRO_UNREFERENCED(length);
		FRAMEPRO_UNREFERENCED(dest_size);
		memcpy(p_dest, p_source, source_length);
		p_dest[source_length] = '\0';
	}

	//------------------------------------------------------------------------
	// this string class is meant to be as light weight as possible and does
	// not clean up its allocations.
	class String
	{
	public:
		//------------------------------------------------------------------------
		String()
		{
		}

		//------------------------------------------------------------------------
		// fast constructor for the case where we are giving it a string literal
		String(const char* p_value)
		:	
#if FRAMEPRO_DETECT_HASH_COLLISIONS
			mp_Value(p_value),
#endif
			m_HashCode(MurmurHash2(p_value))
		{
		}

		//------------------------------------------------------------------------
		// allocate a copy of the string and change mp_Value to point to it
		void TakeCopy(IncrementingBlockAllocator& allocator)
		{
#if FRAMEPRO_DETECT_HASH_COLLISIONS
			const char* p_old_value = mp_Value;
			size_t len = strlen(p_old_value);
			char* p_new_value = (char*)allocator.Alloc(len+1);
			StringCopy(p_new_value, len+1, mp_Value, len);
			mp_Value = p_new_value;
#else
			FRAMEPRO_UNREFERENCED(allocator);
#endif
		}

		//------------------------------------------------------------------------
		FRAMEPRO_FORCE_INLINE unsigned int GetHashCode() const
		{
			return m_HashCode;
		}

		//------------------------------------------------------------------------
		FRAMEPRO_FORCE_INLINE bool operator==(const String& other) const
		{
			return
				m_HashCode == other.m_HashCode
#if FRAMEPRO_DETECT_HASH_COLLISIONS
				&& strcmp(mp_Value, other.mp_Value) == 0
#endif
				;
		}

		//------------------------------------------------------------------------
		// data
	private:
#if FRAMEPRO_DETECT_HASH_COLLISIONS
		const char* mp_Value;
#endif
		unsigned int m_HashCode;
	};

	//------------------------------------------------------------------------
	// this string class is meant to be as light weight as possible and does
	// not clean up its allocations.
	class WString
	{
	public:
		//------------------------------------------------------------------------
		WString()
		{
		}

		//------------------------------------------------------------------------
		// fast constructor for the case where we are giving it a string literal
		WString(const wchar_t* p_value)
		:	
#if FRAMEPRO_DETECT_HASH_COLLISIONS
			mp_Value(p_value),
#endif
			m_HashCode(MurmurHash2(p_value))
		{
		}

		//------------------------------------------------------------------------
		// allocate a copy of the string and change mp_Value to point to it
		void TakeCopy(IncrementingBlockAllocator& allocator)
		{
#if FRAMEPRO_DETECT_HASH_COLLISIONS
			const wchar_t* p_old_value = mp_Value;
			size_t len = wcslen(p_old_value);
			wchar_t* p_new_value = (wchar_t*)allocator.Alloc((len+1)*sizeof(wchar_t));
			StringCopy(p_new_value, len+1, mp_Value, len);
			mp_Value = p_new_value;
#else
			FRAMEPRO_UNREFERENCED(allocator);
#endif
		}

		//------------------------------------------------------------------------
		FRAMEPRO_FORCE_INLINE unsigned int GetHashCode() const
		{
			return m_HashCode;
		}

		//------------------------------------------------------------------------
		FRAMEPRO_FORCE_INLINE bool operator==(const WString& other) const
		{
			return m_HashCode == other.m_HashCode
#if FRAMEPRO_DETECT_HASH_COLLISIONS
				&& wcscmp(mp_Value, other.mp_Value) == 0
#endif
				;
		}

		//------------------------------------------------------------------------
		// data
	private:
#if FRAMEPRO_DETECT_HASH_COLLISIONS
		const wchar_t* mp_Value;
#endif
		unsigned int m_HashCode;
	};

	//------------------------------------------------------------------------
	class DynamicString
	{
	public:
		//------------------------------------------------------------------------
		DynamicString(Allocator* p_allocator)
		:	mp_Value(NULL),
			mp_Allocator(p_allocator)
		{
		}

		//------------------------------------------------------------------------
		void operator=(const char* p_value)
		{
			FRAMEPRO_ASSERT(!mp_Value);
			size_t len = strlen(p_value);
			mp_Value = (char*)mp_Allocator->Alloc(len + 1);
			if(mp_Value)
				StringCopy(mp_Value, len + 1, p_value, len);
		}

		//------------------------------------------------------------------------
		~DynamicString()
		{
			if (mp_Value)
				mp_Allocator->Free(mp_Value);
		}

		//------------------------------------------------------------------------
		void CopyTo(char* p_dest, size_t max_length)
		{
			if (mp_Value)
			{
				size_t len = strlen(mp_Value);
				len = FramePro_Min(len, max_length - 1);
				StringCopy(p_dest, max_length, mp_Value, len);
			}
			else
			{
				StringCopy(p_dest, max_length, "");
			}
		}

		//------------------------------------------------------------------------
		// data
	private:
		char* mp_Value;
		Allocator* mp_Allocator;
	};

	//------------------------------------------------------------------------
	class DynamicWString
	{
	public:
		//------------------------------------------------------------------------
		DynamicWString()
		:	mp_Value(NULL),
			mp_Allocator(NULL)
		{
		}

		//------------------------------------------------------------------------
		~DynamicWString()
		{
			if (mp_Value)
			{
				FRAMEPRO_ASSERT(mp_Allocator);
				mp_Allocator->Free(mp_Value);
			}
		}
		//------------------------------------------------------------------------
		void SetAllocator(Allocator* p_allocator)
		{
			FRAMEPRO_ASSERT(!mp_Allocator);
			FRAMEPRO_ASSERT(p_allocator);

			mp_Allocator = p_allocator;
		}

		//------------------------------------------------------------------------
		void Clear()
		{
			if (mp_Value)
			{
				FRAMEPRO_ASSERT(mp_Allocator);
				mp_Allocator->Free(mp_Value);
				mp_Value = NULL;
			}
		}

		//------------------------------------------------------------------------
		void operator=(const char* p_value)
		{
			FRAMEPRO_ASSERT(!mp_Value);
			FRAMEPRO_ASSERT(mp_Allocator);

			int len = (int)strlen(p_value);
			
			mp_Value = (wchar_t*)mp_Allocator->Alloc((len + 1) * sizeof(wchar_t));
			if (!mp_Value)
				return;

			for(int i = 0; i < len; ++i)
				mp_Value[i] = p_value[i];

			mp_Value[len] = L'\0';
		}

		//------------------------------------------------------------------------
		void operator=(const wchar_t* p_value)
		{
			FRAMEPRO_ASSERT(!mp_Value);
			FRAMEPRO_ASSERT(mp_Allocator);

			int len = (int)wcslen(p_value);

			mp_Value = (wchar_t*)mp_Allocator->Alloc((len + 1) * sizeof(wchar_t));
			if (!mp_Value)
				return;

			for(int i = 0; i < len; ++i)
				mp_Value[i] = p_value[i];
			mp_Value[len] = L'\0';
		}

		//------------------------------------------------------------------------
		const wchar_t* c_str() const
		{
			return mp_Value ? mp_Value : L"";
		}

		//------------------------------------------------------------------------
		// data
	private:
		wchar_t* mp_Value;
		Allocator* mp_Allocator;
	};
}


//------------------------------------------------------------------------
//
// EventTraceWin32.cpp
//

//------------------------------------------------------------------------
#if FRAMEPRO_PLATFORM_WIN

//------------------------------------------------------------------------
#if FRAMEPRO_ENABLE_CONTEXT_SWITCH_TRACKING

//------------------------------------------------------------------------
#if FRAMEPRO_PLATFORM_UE4
	#include "Windows/AllowWindowsPlatformTypes.h"
#endif
#pragma warning(push)
#pragma warning(disable:4668)
#ifndef WIN32_LEAN_AND_MEAN
	#define WIN32_LEAN_AND_MEAN
#endif
#include <windows.h>
#include <wmistr.h>
#define INITGUID
#include <evntrace.h>
#include <evntcons.h>
#include <tchar.h>
#include <Tdh.h>
#pragma warning(pop)
#if FRAMEPRO_PLATFORM_UE4
	#include "Windows/HideWindowsPlatformTypes.h"
#endif

#include <intrin.h>

#pragma comment(lib, "tdh.lib")
#pragma comment(lib, "Advapi32.lib")

//------------------------------------------------------------------------
namespace FramePro
{
	//------------------------------------------------------------------------
	struct ThreadIdKey
	{
		ThreadIdKey() :	m_ThreadId(0) {}

		ThreadIdKey(int thread_id) : m_ThreadId(thread_id) {}

		uint GetHashCode() const
		{
			const unsigned int prime = 0x1000193;
			return (uint)(m_ThreadId * prime);
		}

		bool operator==(const ThreadIdKey& other) const
		{
			return m_ThreadId == other.m_ThreadId;
		}

		int m_ThreadId;
	};

	//------------------------------------------------------------------------
	RelaxedAtomic<bool> g_ShuttingDown = false;		// no way to wait for ETW to stop receiving callbacks after stopping, so we need this horrible global bool

	//------------------------------------------------------------------------
	class EventTraceWin32Imp
	{
	public:
		EventTraceWin32Imp(Allocator* p_allocator);

		~EventTraceWin32Imp();

		bool Start(ContextSwitchCallback p_context_switch_callback, void* p_context_switch_callback_param, DynamicString& error);

		void Stop();

		void Flush();

	private:
		static unsigned long __stdcall TracingThread_Static(LPVOID);

		void TracingThread();

		unsigned long GetEventInformation(PEVENT_RECORD p_event, PTRACE_EVENT_INFO& p_info);

		static VOID WINAPI EventCallback_Static(_In_ PEVENT_RECORD p_event);

		void EventCallback(PEVENT_RECORD p_event);

		//------------------------------------------------------------------------
		// data
	private:
		Allocator* mp_Allocator;

		TRACEHANDLE m_Session;
		TRACEHANDLE m_Consumer;

		CriticalSection m_CriticalSection;
		ContextSwitchCallback m_Callback;
		void* m_CallbackParam;

		HashMap<ThreadIdKey, int> m_ThreadProcessHashMap;

		char m_PropertiesBuffer[sizeof(EVENT_TRACE_PROPERTIES) + sizeof(KERNEL_LOGGER_NAME)];

		void* mp_EventInfoBuffer;
		int m_EventInfoBufferSize;
	};

	//------------------------------------------------------------------------
	EventTraceWin32Imp::EventTraceWin32Imp(Allocator* p_allocator)
	:	mp_Allocator(p_allocator),
		m_Session(0),
		m_Consumer(0),
		m_Callback(NULL),
		m_CallbackParam(NULL),
		m_ThreadProcessHashMap(p_allocator),
		mp_EventInfoBuffer(NULL),
		m_EventInfoBufferSize(0)
	{
		g_ShuttingDown = false;
	}

	//------------------------------------------------------------------------
	EventTraceWin32Imp::~EventTraceWin32Imp()
	{
		if(mp_EventInfoBuffer)
			mp_Allocator->Free(mp_EventInfoBuffer);
	}

	//------------------------------------------------------------------------
	unsigned long EventTraceWin32Imp::GetEventInformation(PEVENT_RECORD p_event, PTRACE_EVENT_INFO& p_info)
	{
		unsigned long buffer_size = 0;
		unsigned long status = TdhGetEventInformation(p_event, 0, NULL, p_info, &buffer_size);

		if (status == ERROR_INSUFFICIENT_BUFFER)
		{
			if((int)buffer_size > m_EventInfoBufferSize)
			{
				mp_Allocator->Free(mp_EventInfoBuffer);
				mp_EventInfoBuffer = mp_Allocator->Alloc(buffer_size);
				if (!mp_EventInfoBuffer)
					return 1;

				FRAMEPRO_ASSERT(mp_EventInfoBuffer);
				m_EventInfoBufferSize = buffer_size;
			}

			p_info = (TRACE_EVENT_INFO*)mp_EventInfoBuffer;

			status = TdhGetEventInformation(p_event, 0, NULL, p_info, &buffer_size);
		}

		return status;
	}

	//------------------------------------------------------------------------
	VOID WINAPI EventTraceWin32Imp::EventCallback_Static(_In_ PEVENT_RECORD p_event)
	{
		if(g_ShuttingDown)
			return;

		EventTraceWin32Imp* p_this = (EventTraceWin32Imp*)p_event->UserContext;
		p_this->EventCallback(p_event);
	}

	//------------------------------------------------------------------------
	void EventTraceWin32Imp::EventCallback(PEVENT_RECORD p_event)
	{
		CriticalSectionScope lock(m_CriticalSection);

		if(!m_Callback)
			return;

		PTRACE_EVENT_INFO p_info = NULL;
		unsigned long status = GetEventInformation(p_event, p_info);

		// check to see this is an MOF class and that it is the context switch event (36)
		if (status == ERROR_SUCCESS && DecodingSourceWbem == p_info->DecodingSource && p_event->EventHeader.EventDescriptor.Opcode == 36)
		{
			PROPERTY_DATA_DESCRIPTOR desc = {0};
			desc.ArrayIndex = ULONG_MAX;

			unsigned long result = 0;

			desc.PropertyName = (ULONGLONG)L"OldThreadId";
			int old_thread_id = 0;
			result = TdhGetProperty(p_event, 0, NULL, 1, &desc, sizeof(old_thread_id), (PBYTE)&old_thread_id);
			FRAMEPRO_ASSERT(result == ERROR_SUCCESS);

			desc.PropertyName = (ULONGLONG)L"NewThreadId";
			int new_thread_id = 0;
			result = TdhGetProperty(p_event, 0, NULL, 1, &desc, sizeof(new_thread_id), (PBYTE)&new_thread_id);
			FRAMEPRO_ASSERT(result == ERROR_SUCCESS);

			desc.PropertyName = (ULONGLONG)L"OldThreadState";
			char old_thread_state = 0;
			result = TdhGetProperty(p_event, 0, NULL, 1, &desc, sizeof(old_thread_state), (PBYTE)&old_thread_state);
			FRAMEPRO_ASSERT(result == ERROR_SUCCESS);

			desc.PropertyName = (ULONGLONG)L"OldThreadWaitReason";
			char old_thread_wait_reason = 0;
			result = TdhGetProperty(p_event, 0, NULL, 1, &desc, sizeof(old_thread_wait_reason), (PBYTE)&old_thread_wait_reason);
			FRAMEPRO_ASSERT(result == ERROR_SUCCESS);

			// the event header process id never seem to be set, so we work it out from the thread id
			int process_id = -1;
			int process_thread_id = new_thread_id ? new_thread_id : old_thread_id;
			if(process_thread_id)
			{
				if(!m_ThreadProcessHashMap.TryGetValue(process_thread_id, process_id))
				{
					HANDLE thread = OpenThread(THREAD_QUERY_INFORMATION, false, process_thread_id);
					if(thread)
					{
						process_id = GetProcessIdOfThread(thread);
						CloseHandle(thread);
					}

					if (!m_ThreadProcessHashMap.Add(process_thread_id, process_id))
						return;
				}
			}

			ContextSwitch context_switch;
			context_switch.m_Timestamp = p_event->EventHeader.TimeStamp.QuadPart;
			context_switch.m_ProcessId = process_id;
#if _MSC_VER > 1600
			context_switch.m_CPUId = p_event->BufferContext.ProcessorIndex;
#else
			context_switch.m_CPUId = p_event->BufferContext.ProcessorNumber;
#endif
			context_switch.m_OldThreadId = old_thread_id;
			context_switch.m_NewThreadId = new_thread_id;
			context_switch.m_OldThreadState = (ThreadState::Enum)old_thread_state;
			context_switch.m_OldThreadWaitReason = (ThreadWaitReason::Enum)old_thread_wait_reason;

			m_Callback(context_switch, m_CallbackParam);
		}
	}

	//------------------------------------------------------------------------
	unsigned long __stdcall EventTraceWin32Imp::TracingThread_Static(LPVOID p_param)
	{
		EventTraceWin32Imp* p_this = (EventTraceWin32Imp*)p_param;
		p_this->TracingThread();
		return 0;
	}

	//------------------------------------------------------------------------
	void EventTraceWin32Imp::TracingThread()
	{
		FRAMEPRO_SET_THREAD_NAME("FramePro ETW Processing Thread");

		ProcessTrace(&m_Consumer, 1, 0, 0);
	}

	//------------------------------------------------------------------------
	void ErrorCodeToString(ULONG error_code, DynamicString& error_string)
	{
		switch (error_code)
		{
			case ERROR_BAD_LENGTH:
				error_string = "ERROR_BAD_LENGTH";
				break;

			case ERROR_INVALID_PARAMETER:
				error_string = "ERROR_INVALID_PARAMETER";
				break;

			case ERROR_ALREADY_EXISTS:
				error_string = "ERROR_ALREADY_EXISTS. Please check that there isn't another application running which is tracing context switches";
				break;

			case ERROR_BAD_PATHNAME:
				error_string = "ERROR_BAD_PATHNAME";
				break;

			case ERROR_DISK_FULL:
				error_string = "ERROR_DISK_FULL";
				break;

			case ERROR_ACCESS_DENIED:
				error_string = "ERROR_ACCESS_DENIED. Please make sure you are running your application with administrator privileges";
				break;

			default:
			{
				char temp[128];
				sprintf_s(temp, "Error code: %lu", error_code);
				error_string = temp;
			}
		}
	}

	//------------------------------------------------------------------------
	bool EventTraceWin32Imp::Start(ContextSwitchCallback p_context_switch_callback, void* p_context_switch_callback_param, DynamicString& error)
	{
		// only one kernal session allowed, so much stop any currently running session first
		Stop();

		{
			CriticalSectionScope lock(m_CriticalSection);
			m_Callback = p_context_switch_callback;
			m_CallbackParam = p_context_switch_callback_param;
		}

		// session name is stored at the end of the properties struct
		size_t properties_buffer_size = sizeof(m_PropertiesBuffer);
		char* p_properties_mem = m_PropertiesBuffer;
		memset(p_properties_mem, 0, properties_buffer_size);

		// initialise the session properties
		EVENT_TRACE_PROPERTIES* p_properties = (EVENT_TRACE_PROPERTIES*)p_properties_mem;

		p_properties->Wnode.BufferSize = (ULONG)properties_buffer_size;
		p_properties->Wnode.Flags = WNODE_FLAG_TRACED_GUID;
		p_properties->Wnode.Guid = SystemTraceControlGuid;		// GUID for a NT Kernel Logger session
		p_properties->Wnode.ClientContext = 1;					// Clock resolution: use query performance counter (QPC)

		p_properties->EnableFlags = EVENT_TRACE_FLAG_CSWITCH;
		p_properties->LogFileMode = EVENT_TRACE_REAL_TIME_MODE;
		p_properties->LoggerNameOffset = sizeof(EVENT_TRACE_PROPERTIES);

		// start the new session
		ULONG result = StartTrace(&m_Session, KERNEL_LOGGER_NAME, p_properties);
		if (result != ERROR_SUCCESS)
		{
			ErrorCodeToString(result, error);
			return false;
		}

		// open the session
		EVENT_TRACE_LOGFILE log_file = {0};
		log_file.LoggerName = (FRAMEPRO_TCHAR*)KERNEL_LOGGER_NAME;
		log_file.ProcessTraceMode = PROCESS_TRACE_MODE_EVENT_RECORD | PROCESS_TRACE_MODE_RAW_TIMESTAMP | PROCESS_TRACE_MODE_REAL_TIME;
		log_file.EventRecordCallback = EventCallback_Static;
		log_file.Context = this;

		m_Consumer = OpenTrace(&log_file);
		if (m_Consumer == INVALID_PROCESSTRACE_HANDLE)
		{
			error = "OpenTrace() failed";
			return false;
		}

		// start the processing thread
		HANDLE thread = CreateThread(0, 0, TracingThread_Static, this, 0, NULL);
		if (thread == NULL)
		{
			error = "CreateThread returned NULL";
			return false;
		}
		CloseHandle(thread);

		return true;
	}

	//------------------------------------------------------------------------
	void EventTraceWin32Imp::Stop()
	{
		size_t properties_buffer_size = sizeof(m_PropertiesBuffer);
		char* p_properties_mem = m_PropertiesBuffer;
		memset(p_properties_mem, 0, properties_buffer_size);

		EVENT_TRACE_PROPERTIES* p_properties = (EVENT_TRACE_PROPERTIES*)p_properties_mem;

		p_properties->Wnode.BufferSize = (ULONG)properties_buffer_size;
		p_properties->Wnode.Guid = SystemTraceControlGuid;		// GUID for a NT Kernel Logger session
		p_properties->LoggerNameOffset = sizeof(EVENT_TRACE_PROPERTIES);

		_tcscpy_s((FRAMEPRO_TCHAR*)((char*)p_properties + p_properties->LoggerNameOffset), sizeof(KERNEL_LOGGER_NAME)/sizeof(FRAMEPRO_TCHAR), KERNEL_LOGGER_NAME);

		// stop any old sessions that were not stopped
		ControlTrace(0, KERNEL_LOGGER_NAME, p_properties, EVENT_TRACE_CONTROL_STOP);

		m_Session = 0;

		if(m_Consumer)
		{
			CloseTrace(m_Consumer);
			m_Consumer = 0;
		}

		{
			CriticalSectionScope lock(m_CriticalSection);
			m_Callback = NULL;
			m_CallbackParam = NULL;
		}
	}

	//------------------------------------------------------------------------
	void EventTraceWin32Imp::Flush()
	{
		if(!m_Session)
			return;

		size_t properties_buffer_size = sizeof(m_PropertiesBuffer);
		char* p_properties_mem = m_PropertiesBuffer;
		memset(p_properties_mem, 0, properties_buffer_size);

		EVENT_TRACE_PROPERTIES* p_properties = (EVENT_TRACE_PROPERTIES*)p_properties_mem;

		p_properties->Wnode.BufferSize = (ULONG)properties_buffer_size;
		p_properties->Wnode.Guid = SystemTraceControlGuid;		// GUID for a NT Kernel Logger session
		p_properties->LoggerNameOffset = sizeof(EVENT_TRACE_PROPERTIES);

		_tcscpy_s((FRAMEPRO_TCHAR*)((char*)p_properties + p_properties->LoggerNameOffset), sizeof(KERNEL_LOGGER_NAME)/sizeof(FRAMEPRO_TCHAR), KERNEL_LOGGER_NAME);

		#if FRAMEPRO_DEBUG
			ULONG result = ControlTrace(m_Session, NULL, p_properties, EVENT_TRACE_CONTROL_FLUSH);
			FRAMEPRO_ASSERT(result == ERROR_SUCCESS);
		#else
			ControlTrace(m_Session, NULL, p_properties, EVENT_TRACE_CONTROL_FLUSH);
		#endif
	}
}

//------------------------------------------------------------------------
namespace FramePro
{
	//------------------------------------------------------------------------
	EventTraceWin32::EventTraceWin32(Allocator* p_allocator)
	:	mp_Imp(New<EventTraceWin32Imp>(p_allocator, p_allocator)),
		mp_Allocator(p_allocator)
	{
	}

	//------------------------------------------------------------------------
	EventTraceWin32::~EventTraceWin32()
	{
		g_ShuttingDown = true;

		Delete(mp_Allocator, mp_Imp);
	}

	//------------------------------------------------------------------------
	bool EventTraceWin32::Start(ContextSwitchCallback p_context_switch_callback, void* p_context_switch_callback_param, DynamicString& error)
	{
		return mp_Imp->Start(p_context_switch_callback, p_context_switch_callback_param, error);
	}

	//------------------------------------------------------------------------
	void EventTraceWin32::Stop()
	{
		mp_Imp->Stop();
	}

	//------------------------------------------------------------------------
	void EventTraceWin32::Flush()
	{
		mp_Imp->Flush();
	}

	//------------------------------------------------------------------------
	void* EventTraceWin32::Create(Allocator* p_allocator)
	{
		return New<EventTraceWin32>(p_allocator, p_allocator);
	}

	//------------------------------------------------------------------------
	void EventTraceWin32::Destroy(void* p_context_switch_recorder, Allocator* p_allocator)
	{
		if (p_context_switch_recorder)
		{
			EventTraceWin32* p_event_trace_win32 = (EventTraceWin32*)p_context_switch_recorder;
			Delete(p_allocator, p_event_trace_win32);
		}
	}

	//------------------------------------------------------------------------
	bool EventTraceWin32::Start(
		void* p_context_switch_recorder,
		Platform::ContextSwitchCallbackFunction p_callback,
		void* p_context,
		DynamicString& error)
	{
		if (!p_context_switch_recorder)
			return false;

		EventTraceWin32* p_event_trace_win32 = (EventTraceWin32*)p_context_switch_recorder;

		bool started = p_event_trace_win32->Start(p_callback, p_context, error);

		if(!started)
			FramePro::DebugWrite("FramePro Warning: Failed to start recording context switches. Please make sure that you are running with administrator privileges.\n");

		return started;
	}

	//------------------------------------------------------------------------
	void EventTraceWin32::Stop(void* p_context_switch_recorder)
	{
		if (p_context_switch_recorder)
		{
			EventTraceWin32* p_event_trace_win32 = (EventTraceWin32*)p_context_switch_recorder;
			p_event_trace_win32->Stop();
		}
	}

	//------------------------------------------------------------------------
	void EventTraceWin32::Flush(void* p_context_switch_recorder)
	{
		if (p_context_switch_recorder)
		{
			EventTraceWin32* p_event_trace_win32 = (EventTraceWin32*)p_context_switch_recorder;
			p_event_trace_win32->Flush();
		}
	}
}

//------------------------------------------------------------------------
#endif			// #if FRAMEPRO_ENABLE_CONTEXT_SWITCH_TRACKING

//------------------------------------------------------------------------
#endif		// #if FRAMEPRO_PLATFORM_WIN


//------------------------------------------------------------------------
//
// FrameProTLS.h
//

//------------------------------------------------------------------------

//------------------------------------------------------------------------
//
// Socket.h
//

//------------------------------------------------------------------------
#if FRAMEPRO_SOCKETS_ENABLED

//------------------------------------------------------------------------
namespace FramePro
{
	//------------------------------------------------------------------------
	class SocketImp;

	//------------------------------------------------------------------------
	class Socket
	{
	public:
		Socket();

		~Socket();

		void Disconnect();

		bool Bind(const char* p_port);

		bool StartListening();

		bool Accept(Socket& client_socket);

		int Receive(void* p_buffer, int size);

		bool Send(const void* p_buffer, size_t size);

		bool IsValid() const;

		//------------------------------------------------------------------------
		// data
	private:
		static const int m_OSSocketMemMaxSize = 8;
		char m_OSSocketMem[m_OSSocketMemMaxSize];

		bool m_Listening;
	};
}

//------------------------------------------------------------------------
#endif		// #if FRAMEPRO_SOCKETS_ENABLED


//------------------------------------------------------------------------
//
// Packets.h
//

//------------------------------------------------------------------------
namespace FramePro
{
	//------------------------------------------------------------------------
	// send packets
	//------------------------------------------------------------------------

	//------------------------------------------------------------------------
	#ifdef __clang__
		#pragma clang diagnostic push
		#pragma clang diagnostic ignored "-Wunused-private-field"
	#endif

	//------------------------------------------------------------------------
	namespace CustomStatValueType
	{
		enum Enum
		{
			Int64 = 0,
			Double,
		};
	}

	//------------------------------------------------------------------------
	struct ConnectPacket
	{
		ConnectPacket(int64 clock_frequency, int process_id, Platform::Enum platform)
		:	m_PacketType(PacketType::Connect),
			m_Version(g_FrameProLibVersion),
			m_ClockFrequency(clock_frequency),
			m_ProcessId(process_id),
			m_Platform(platform)
		{
		}

	private:
		PacketType::Enum m_PacketType;
		int m_Version;
		int64 m_ClockFrequency;
		int m_ProcessId;
		Platform::Enum m_Platform;
	};

	//------------------------------------------------------------------------
	struct SessionInfoPacket
	{
		SessionInfoPacket(StringId name, StringId value)
		:	m_PacketType(PacketType::SessionInfoPacket),
			m_Padding(0),
			m_Name(name),
			m_Value(value)
		{
		}

	private:
		PacketType::Enum m_PacketType;
		int m_Padding;
		StringId m_Name;
		StringId m_Value;
	};

	//------------------------------------------------------------------------
	struct TimeSpanPacket
	{
		int m_PacketType_AndCore;
		int m_ThreadId;
		StringId m_NameAndSourceInfo;
		int64 m_StartTime;
		int64 m_EndTime;
	};

	//------------------------------------------------------------------------
	struct TimeSpanCustomStatPacket
	{
		int m_PacketType;
		int m_ThreadId;
		int m_ValueType;	// CustomStatValueType enum
		int m_Padding;
		StringId m_Name;
		int64 m_ValueInt64;
		double m_ValueDouble;
		int64 m_Time;
	};

	//------------------------------------------------------------------------
	struct NamedTimeSpanPacket
	{
		int m_PacketType_AndCore;
		int m_ThreadId;
		int64 m_Name;
		StringId m_SourceInfo;
		int64 m_StartTime;
		int64 m_EndTime;
	};

	//------------------------------------------------------------------------
	struct FrameStartPacket
	{
		FrameStartPacket(int64 frame_start_time, int64 wait_for_send_complete_time)
		:	m_PacketType(PacketType::FrameStart),
			m_Legacy1(0),
			m_Legacy2(0),
			m_Padding(0xffffffff),
			m_FrameStartTime(frame_start_time),
			m_WaitForSendCompleteTime(wait_for_send_complete_time),
			m_Legacy4(0)
		{
		}

	private:
		PacketType::Enum m_PacketType;
		int m_Legacy1;
		int m_Legacy2;
		int m_Padding;
		int64 m_FrameStartTime;
		int64 m_WaitForSendCompleteTime;
		int64 m_Legacy4;
	};

	//------------------------------------------------------------------------
	struct ThreadNamePacket
	{
	public:
		ThreadNamePacket(int thread_id, int64 name)
		:	m_PacketType(PacketType::ThreadName),
			m_ThreadID(thread_id),
			m_Name(name)
		{
		}

	private:
		PacketType::Enum m_PacketType;
		int m_ThreadID;
		int64 m_Name;
	};

	//------------------------------------------------------------------------
	struct ThreadOrderPacket
	{
	public:
		ThreadOrderPacket(StringId thread_name)
		:	m_PacketType(PacketType::ThreadOrder),
			m_Padding(0xffffffff),
			m_ThreadName(thread_name)
		{
		}

	private:
		PacketType::Enum m_PacketType;
		int m_Padding;
		StringId m_ThreadName;
	};

	//------------------------------------------------------------------------
	struct StringPacket
	{
		PacketType::Enum m_PacketType;
		int m_Length;			// length in chars
		StringId m_StringId;
		// name string follows in buffer
	};

	//------------------------------------------------------------------------
	struct MainThreadPacket
	{
		MainThreadPacket(int thread_id)
		:	m_PacketType(PacketType::MainThreadPacket),
			m_ThreadId(thread_id)
		{
		}

	private:
		PacketType::Enum m_PacketType;
		int m_ThreadId;
	};

	//------------------------------------------------------------------------
	struct SessionStatsPacket
	{
		SessionStatsPacket()
		:	m_PacketType(PacketType::SessionStatsPacket),
			m_Padding(0xffffffff),
			m_SendBufferSize(0),
			m_StringMemorySize(0),
			m_MiscMemorySize(0),
			m_RecordingFileSize(0)
		{
		}

		PacketType::Enum m_PacketType;
		int m_Padding;
		int64 m_SendBufferSize;
		int64 m_StringMemorySize;
		int64 m_MiscMemorySize;
		int64 m_RecordingFileSize;
	};

	//------------------------------------------------------------------------
	struct ContextSwitchPacket
	{
		PacketType::Enum m_PacketType;
		int m_CPUId;
		int64 m_Timestamp;
		int m_ProcessId;
		int m_OldThreadId;
		int m_NewThreadId;
		int m_OldThreadState;
		int m_OldThreadWaitReason;
		int m_Padding;
	};

	//------------------------------------------------------------------------
	struct ContextSwitchRecordingStartedPacket
	{
		PacketType::Enum m_PacketType;
		int m_StartedSucessfully;		// bool
		char m_Error[FRAMEPRO_MAX_INLINE_STRING_LENGTH];
	};

	//------------------------------------------------------------------------
	struct ProcessNamePacket
	{
		ProcessNamePacket(int process_id, int64 name_id)
		:	m_PacketType(PacketType::ProcessNamePacket),
			m_ProcessId(process_id),
			m_NameId(name_id)
		{
		}

		PacketType::Enum m_PacketType;
		int m_ProcessId;
		int64 m_NameId;
	};

	//------------------------------------------------------------------------
	struct CustomStatPacketInt64
	{
		uint m_PacketTypeAndValueType;
		int m_Count;
		StringId m_Name;
		int64 m_Value;
		int64 m_Time;
	};

	//------------------------------------------------------------------------
	struct CustomStatPacketDouble
	{
		uint m_PacketTypeAndValueType;
		int m_Count;
		StringId m_Name;
		double m_Value;
		int64 m_Time;
	};

	//------------------------------------------------------------------------
	struct HiResTimerScopePacket
	{
		PacketType::Enum m_PacketType;
		int m_Padding;
		int64 m_StartTime;
		int64 m_EndTime;
		int m_Count;
		int m_ThreadId;
		// array of HiResTimer follows

		struct HiResTimer
		{
			StringId m_Name;
			int64 m_Duration;
			int64 m_Count;
		};
	};

	//------------------------------------------------------------------------
	struct LogPacket
	{
		PacketType::Enum m_PacketType;
		int m_Length;			// length in chars
		int64 m_Time;
		// name string follows in buffer
	};

	//------------------------------------------------------------------------
	struct EventPacket
	{
		PacketType::Enum m_PacketType;
		uint m_Colour;
		StringId m_Name;
		int64 m_Time;
	};

	//------------------------------------------------------------------------
	struct WaitEventPacket
	{
		PacketType::Enum m_PacketType;
		int m_Thread;
		int m_Core;
		int m_Padding;
		int64 m_EventId;
		int64 m_Time;
	};

	//------------------------------------------------------------------------
	struct CallstackPacket
	{
		PacketType::Enum m_PacketType;
		int m_CallstackId;
		int m_CallstackSize;	// size of the callstack that follows in the send buffer, or 0 if we have already sent this callstack
	};

	//------------------------------------------------------------------------
	struct TimeSpanCallstackPacket
	{
		// we don't have a packet type here because it always follows a time span packet
		int m_CallstackId;
		int m_CallstackSize;	// size of the callstack that follows in the send buffer, or 0 if we have already sent this callstack
	};

	//------------------------------------------------------------------------
	struct ScopeColourPacket
	{
		PacketType::Enum m_PacketType;
		uint m_Colour;
		StringId m_Name;
	};

	//------------------------------------------------------------------------
	struct CustomStatInfoPacket
	{
		PacketType::Enum m_PacketType;
		uint m_Padding;
		StringId m_Name;
		StringId m_Value;
	};

	//------------------------------------------------------------------------
	struct CustomStatColourPacket
	{
		PacketType::Enum m_PacketType;
		uint m_Colour;
		StringId m_Name;
	};

	//------------------------------------------------------------------------
	// receive packets
	//------------------------------------------------------------------------

	//------------------------------------------------------------------------
	struct RequestStringLiteralPacket
	{
		StringId m_StringId;
		int m_StringLiteralType;
		int m_Padding;
	};

	//------------------------------------------------------------------------
	struct SetConditionalScopeMinTimePacket
	{
		int m_MinTime;
	};

	//------------------------------------------------------------------------
	struct ConnectResponsePacket
	{
		int m_Interactive;
		int m_RecordContextSwitches;
	};

	//------------------------------------------------------------------------
	struct RequestRecordedDataPacket
	{
	};

	//------------------------------------------------------------------------
	struct SetCallstackRecordingEnabledPacket
	{
		int m_Enabled;
	};

	//------------------------------------------------------------------------
	#ifdef __clang__
		#pragma clang diagnostic pop
	#endif
}


//------------------------------------------------------------------------
//
// PointerSet.h
//

//------------------------------------------------------------------------
#define FRAMEPRO_PRIME 0x01000193

//------------------------------------------------------------------------
namespace FramePro
{
	//------------------------------------------------------------------------
	class Allocator;

	//------------------------------------------------------------------------
	class PointerSet
	{
	public:
		PointerSet(Allocator* p_allocator);

		~PointerSet();

		size_t GetMemorySize() const { return m_Capacity * sizeof(const void*); }

		//------------------------------------------------------------------------
		// return true if added, false if already in set
		FRAMEPRO_FORCE_INLINE bool Add(const void* p)
		{
#if FRAMEPRO_X64
			unsigned int hash = (unsigned int)((unsigned long long)p * 18446744073709551557UL);
#else
			unsigned int hash = (unsigned int)p * 4294967291;
#endif
			int index = hash & m_CapacityMask;

			// common case handled inline
			const void* p_existing = mp_Data[index];
			if(p_existing == p)
				return false;

			return AddInternal(p, hash, index);
		}

	private:
		bool Grow();

		bool AddInternal(const void* p, int64 hash, int index);

		//------------------------------------------------------------------------
		// data
	private:
		const void** mp_Data;
		unsigned int m_CapacityMask;
		int m_Count;
		int m_Capacity;

		Allocator* mp_Allocator;
	};
}


//------------------------------------------------------------------------
//
// SendBuffer.h
//

//------------------------------------------------------------------------
namespace FramePro
{
	//------------------------------------------------------------------------
	class Allocator;
	class FrameProTLS;

	//------------------------------------------------------------------------
	class SendBuffer
	{
	public:
		SendBuffer(Allocator* p_allocator, int capacity, FrameProTLS* p_owner);

		~SendBuffer();

		const void* GetBuffer() const { return mp_Buffer; }

		bool AllocateBuffer(int capacity);

		void ClearBuffer();

		void ClearSize() { m_Size = 0; }

		int GetSize() const { return m_Size; }

		int GetCapacity() const { return m_Capacity; }

		SendBuffer* GetNext() const { return mp_Next; }

		void SetNext(SendBuffer* p_next) { mp_Next = p_next; }

		void Swap(void*& p_buffer, int& size, int capacity)
		{
			FramePro::Swap(mp_Buffer, p_buffer);
			FramePro::Swap(m_Size, size);
			m_Capacity = capacity;
		}

		void Swap(SendBuffer* p_send_buffer)
		{
			FramePro::Swap(mp_Buffer, p_send_buffer->mp_Buffer);
			FramePro::Swap(m_Size, p_send_buffer->m_Size);
			FramePro::Swap(m_Capacity, p_send_buffer->m_Capacity);
		}

		FrameProTLS* GetOwner() { return mp_Owner; }

		int64 GetCreationTime() const { return m_CreationTime; }

		void SetCreationTime();

		//------------------------------------------------------------------------
		// data
	private:
		void* mp_Buffer;
		int m_Size;

		int m_Capacity;

		SendBuffer* mp_Next;

		Allocator* mp_Allocator;

		FrameProTLS* mp_Owner;

		int64 m_CreationTime;
	};
}


//------------------------------------------------------------------------
//
// Buffer.h
//

//------------------------------------------------------------------------
namespace FramePro
{
	//--------------------------------------------------------------------
	class Buffer
	{
	public:
		Buffer()
		:	mp_Buffer(NULL),
			m_Size(0),
			m_Capacity(0),
			mp_Allocator(NULL)
		{
		}

		Buffer(Allocator* p_allocator)
			:	mp_Buffer(NULL),
			m_Size(0),
			m_Capacity(0),
			mp_Allocator(p_allocator)
		{
		}

		~Buffer()
		{
			if(mp_Buffer)
				mp_Allocator->Free(mp_Buffer);
		}

		void SetAllocator(Allocator* p_allocator) { mp_Allocator = p_allocator; }

		void* GetBuffer() const { return mp_Buffer; }

		int GetSize() const { return m_Size; }

		int GetMemorySize() const { return m_Capacity; }

		void Clear()
		{
			m_Size = 0;
		}

		void ClearAndFree()
		{
			Clear();

			if(mp_Buffer)
			{
				mp_Allocator->Free(mp_Buffer);
				mp_Buffer = NULL;
			}
		}

		void* Allocate(int size)
		{
			int old_size = m_Size;
			int new_size = old_size + size;
			if(new_size > m_Capacity)
			{
				int double_capacity = 2*m_Capacity;
				if (!Resize(double_capacity > new_size ? double_capacity : new_size))
					return NULL;
			}
			void* p = (char*)mp_Buffer + old_size;
			m_Size = new_size;

			return p;
		}

	private:
		bool Resize(int new_capacity)
		{
			void* p_new_buffer = mp_Allocator->Alloc(new_capacity);
			if (!p_new_buffer)
			{
				mp_Buffer = NULL;
				return false;
			}

			int current_size = m_Size;
			if(current_size)
				memcpy(p_new_buffer, mp_Buffer, current_size);

			mp_Allocator->Free(mp_Buffer);
			mp_Buffer = p_new_buffer;

			m_Capacity = new_capacity;

			return true;
		}

		//------------------------------------------------------------------------
		// data
	private:
		void* mp_Buffer;
		
		int m_Size;
		int m_Capacity;

		Allocator* mp_Allocator;
	};
}


//------------------------------------------------------------------------
//
// List.h
//

//------------------------------------------------------------------------
namespace FramePro
{
	//------------------------------------------------------------------------
	template<typename T>
	class List
	{
	public:
		//------------------------------------------------------------------------
		List()
		:	mp_Head(NULL),
			mp_Tail(NULL)
		{
		}

		//------------------------------------------------------------------------
		bool IsEmpty() const
		{
			return mp_Head == NULL;
		}

		//------------------------------------------------------------------------
		void Clear()
		{
			mp_Head = mp_Tail = NULL;
			CheckState();
		}

		//------------------------------------------------------------------------
		T* GetHead()
		{
			return mp_Head;
		}

		//------------------------------------------------------------------------
		const T* GetHead() const
		{
			return mp_Head;
		}

		//------------------------------------------------------------------------
		void AddHead(T* p_item)
		{
			FRAMEPRO_ASSERT(!p_item->GetNext());
			p_item->SetNext(mp_Head);
			mp_Head = p_item;
			if(!mp_Tail)
				mp_Tail = p_item;

			CheckState();
		}

		//------------------------------------------------------------------------
		T* RemoveHead()
		{
			T* p_item = mp_Head;
			T* p_new_head = p_item->GetNext();
			mp_Head = p_new_head;
			p_item->SetNext(NULL);
			if(!p_new_head)
				mp_Tail = NULL;
			CheckState();
			return p_item;
		}

		//------------------------------------------------------------------------
		void AddTail(T* p_item)
		{
			FRAMEPRO_ASSERT(!p_item->GetNext());

			if(mp_Tail)
			{
				FRAMEPRO_ASSERT(mp_Head);
				mp_Tail->SetNext(p_item);
			}
			else
			{
				mp_Head = p_item;
			}

			mp_Tail = p_item;

			CheckState();
		}

		//------------------------------------------------------------------------
		void MoveAppend(List<T>& list)
		{
			if(list.IsEmpty())
				return;

			T* p_head = list.GetHead();

			if(mp_Tail)
				mp_Tail->SetNext(p_head);
			else
				mp_Head = p_head;

			mp_Tail = list.mp_Tail;

			list.Clear();
			list.CheckState();

			CheckState();
		}

		//------------------------------------------------------------------------
		void Remove(T* p_item)
		{
			T* p_prev = NULL;
			for(T* p_iter=mp_Head; p_iter && p_iter!=p_item; p_iter=p_iter->GetNext())
				p_prev = p_iter;
			FRAMEPRO_ASSERT(!p_prev || p_prev->GetNext());
			
			T* p_next = p_item->GetNext();
			if(p_prev)
				p_prev->SetNext(p_item->GetNext());
			else
				mp_Head = p_next;

			if(mp_Tail == p_item)
				mp_Tail = p_prev;

			p_item->SetNext(NULL);

			CheckState();
		}

		//------------------------------------------------------------------------
		void CheckState()
		{
			FRAMEPRO_ASSERT((!mp_Head && !mp_Tail) || (mp_Head && mp_Tail));

			#if FRAMEPRO_DEBUG
				T* p_tail = mp_Head;
				while (p_tail && p_tail->GetNext())
					p_tail = p_tail->GetNext();
				FRAMEPRO_ASSERT(mp_Tail == p_tail);
			#endif
		}

		//------------------------------------------------------------------------
		// data
	private:
		T* mp_Head;
		T* mp_Tail;
	};
}


//------------------------------------------------------------------------
//
// ConditionalParentScope.h
//

//------------------------------------------------------------------------
namespace FramePro
{
	//------------------------------------------------------------------------
	class ConditionalParentScope
	{
	public:
		ConditionalParentScope(const char* p_name)
		:	mp_Name(p_name),
			m_PreDuration(0),
			m_PostDuration(0),
			mp_SendBuffer(NULL),
			mp_Next(NULL),
			m_LastPopConditionalChildrenTime(0)
		{
		}

		ConditionalParentScope* GetNext() const { return mp_Next; }

		void SetNext(ConditionalParentScope* p_next) { mp_Next = p_next; }

		// data
		const char* mp_Name;
		int64 m_PreDuration;					// in ms
		int64 m_PostDuration;					// in ms
		SendBuffer* mp_SendBuffer;					// only accessed by TLS thread
		List<SendBuffer> m_ChildSendBuffers;		// accessed from multiple threads
		ConditionalParentScope* mp_Next;
		int64 m_LastPopConditionalChildrenTime;
	};
}


//------------------------------------------------------------------------
//
// FrameProCallstackSet.h
//

//------------------------------------------------------------------------
namespace FramePro
{
	//------------------------------------------------------------------------
	struct Callstack
	{
		uint64* mp_Stack;
		int m_ID;
		int m_Size;
		unsigned int m_Hash;
	};

	//------------------------------------------------------------------------
	// A hash set collection for Callstack structures. Callstacks are added and
	// retreived using the stack address array as the key.
	// This class only allocates memory using virtual alloc/free to avoid going
	// back into the mian allocator.
	class CallstackSet
	{
	public:
		CallstackSet(Allocator* p_allocator);

		~CallstackSet();

		Callstack* Get(uint64* p_stack, int stack_size, unsigned int hash);

		Callstack* Add(uint64* p_stack, int stack_size, unsigned int hash);

		void Clear();

	private:
		void Grow();

		void Add(Callstack* p_callstack);

		//------------------------------------------------------------------------
		// data
	private:
		Callstack** mp_Data;
		unsigned int m_CapacityMask;
		int m_Count;
		int m_Capacity;

		Allocator* mp_Allocator;
		IncrementingBlockAllocator m_BlockAllocator;

		static std::atomic<int> m_NextUniqueId;
	};
}


//------------------------------------------------------------------------
//
// FrameProStackTrace.h
//

//------------------------------------------------------------------------
#if FRAMEPRO_ENABLE_CALLSTACKS

//------------------------------------------------------------------------
namespace FramePro
{
	//------------------------------------------------------------------------
	struct CallstackResult
	{
		CallstackResult()
		:	mp_Callstack(NULL),
			m_IsNew(false)
		{
		}

		Callstack* mp_Callstack;
		bool m_IsNew;
	};

	//------------------------------------------------------------------------
	class StackTrace
	{
	public:
		StackTrace(Allocator* p_allocator);

		void Clear();

		CallstackResult Capture();

		//------------------------------------------------------------------------
		// data
	private:
		void* m_Stack[FRAMEPRO_STACK_TRACE_SIZE];

		int m_StackCount;
		unsigned int m_StackHash;

		CallstackSet m_CallstackSet;
	};
}

//------------------------------------------------------------------------
#endif		// #if FRAMEPRO_ENABLE_CALLSTACKS


//------------------------------------------------------------------------
namespace FramePro
{
	//------------------------------------------------------------------------
	class FrameProTLS
	{
		struct HiResTimer
		{
			const char* mp_Name;
			int64 m_Duration;
			int64 m_Count;
		};

		struct HiResTimerList
		{
			Array<HiResTimer> m_Timers;

			HiResTimerList* mp_Prev;
			HiResTimerList* mp_Next;

			HiResTimerList* GetPrev() { return mp_Prev; }
			HiResTimerList* GetNext() { return mp_Next; }
			void SetPrev(HiResTimerList* p_prev) { mp_Prev = p_prev; }
			void SetNext(HiResTimerList* p_next) { mp_Next = p_next; }
		};

	public:
		FrameProTLS(Allocator* p_allocator, int64 clock_frequency);

		~FrameProTLS();

		//---------------------------------------------------
		// these functions are called from the main thread, so care needs to be taken with thread safety

		bool OnConnected(bool recording_to_file);

		void OnDisconnected();

		void SendSessionInfoBuffer();

		bool OnFrameStart();

		void FlushSendBuffers();

		void LockSessionInfoBuffer() { m_SessionInfoBufferLock.Enter(); }
		
		void UnlockSessionInfoBuffer() { m_SessionInfoBufferLock.Leave(); }

		void SetInteractive(bool value)
		{
			m_Interactive = value;
			UpdateSendStringsImmediatelyFlag();
		}

		FrameProTLS* GetNext() const { return mp_Next; }

		void SetNext(FrameProTLS* p_next) { mp_Next = p_next; }

		size_t GetStringMemorySize() const { return m_StringMemorySize + m_LiteralStringSetMemorySize; }

		size_t GetSendBufferMemorySize() const { return m_SendBufferMemorySize + m_SessionInfoBufferMemorySize; }

		//---------------------------------------------------
		// these function are only called from the TLS thread

		FRAMEPRO_FORCE_INLINE int GetThreadId() const { return m_ThreadId; }

		FRAMEPRO_FORCE_INLINE bool IsInteractive() const { return m_Interactive; }

		FRAMEPRO_FORCE_INLINE void* AllocateSpaceInBuffer(int size)
		{
			FRAMEPRO_ASSERT(m_CurrentSendBufferCriticalSection.Locked());
			FRAMEPRO_ASSERT(IsOnTLSThread() || !g_Connected);		// can only be accessed from TLS thread, unless we haven't connected yet
			FRAMEPRO_ASSERT(size <= m_SendBufferCapacity);

			if(m_CurrentSendBufferSize + size >= m_SendBufferCapacity)
				FlushCurrentSendBuffer_no_lock();

			void* p = (char*)mp_CurrentSendBuffer + m_CurrentSendBufferSize;
			m_CurrentSendBufferSize += size;
			return p;
		}

		template<typename T>
		FRAMEPRO_FORCE_INLINE T* AllocateSpaceInBuffer()
		{
			return (T*)AllocateSpaceInBuffer(sizeof(T));
		}
		
		void SetThreadName(int thread_id, const char* p_name);

		void SetThreadOrder(StringId thread_name);

		void SetMainThread(int main_thread_id);

		StringId RegisterString(const char* p_str);

		StringId RegisterString(const wchar_t* p_str);

		FRAMEPRO_NO_INLINE void SendString(const char* p_string, PacketType::Enum packet_type);

		FRAMEPRO_NO_INLINE void SendString(const wchar_t* p_string, PacketType::Enum packet_type);

		void SendConnectPacket(int64 clock_frequency, int process_id, Platform::Enum platform);

		void SendStringLiteral(StringLiteralType::Enum string_literal_type, StringId string_id);

		void Send(const void* p_data, int size);

		bool SendStringsImmediately() const { return m_SendStringsImmediately; }

		void CollectSendBuffers(List<SendBuffer>& list);

		void AddEmptySendBuffer(SendBuffer* p_send_buffer);

		template<class PacketT>
		void SendSessionInfoPacket(const PacketT& packet)
		{
			SendSessionInfo(&packet, sizeof(packet));
		}

		template<class T> FRAMEPRO_FORCE_INLINE void SendPacket(const T& packet) { Send(&packet, sizeof(packet)); }

		CriticalSection& GetCurrentSendBufferCriticalSection() { return m_CurrentSendBufferCriticalSection; }

		void Shutdown() { m_ShuttingDown = true; }

		bool ShuttingDown() const { return m_ShuttingDown; }

		FRAMEPRO_NO_INLINE bool FlushCurrentSendBuffer();

		void PushConditionalParentScope(const char* p_name, int64 pre_duration, int64 post_duration);
		
		void PopConditionalParentScope(bool add_children);

		void SendLogPacket(const char* p_message);

		void SendEventPacket(const char* p_name, uint colour);

		void SendScopeColourPacket(StringId name, uint colour);
		
		void SendCustomStatGraphPacket(StringId name, StringId graph);

		void SendCustomStatUnitPacket(StringId name, StringId unit);

		void SendCustomStatColourPacket(StringId name, uint colour);

		FRAMEPRO_FORCE_INLINE void StartHiResTimer(const char* p_name)
		{
			FRAMEPRO_ASSERT(IsOnTLSThread());

			#if FRAMEPRO_ALLOW_UNPARENTED_HIRES_SCOPES
				if(!m_ScopeDepth)
					return;
			#else
				FRAMEPRO_ASSERT(m_ScopeDepth);	// all hires scopes must have a parent
			#endif

			// try and find the timer of the specified name
			int count = m_HiResTimers.GetCount();
			HiResTimer* p_timer = NULL;
			int i;
			for (i = 0; i<count; ++i)
			{
				HiResTimer* p_timer_tier = &m_HiResTimers[i];
				if (p_timer_tier->mp_Name == p_name)
				{
					p_timer = p_timer_tier;
					break;
				}
			}

			// add the timer if not found
			if (!p_timer)
			{
				if (!m_HiResTimers.GetCount())
				{
					FRAMEPRO_GET_CLOCK_COUNT(m_FirstHiresTimerTime);
				}

				HiResTimer hires_timer;
				hires_timer.mp_Name = p_name;
				hires_timer.m_Duration = 0;
				hires_timer.m_Count = 0;
				m_HiResTimers.Add(hires_timer);
			}

			// remember the current active timer and set this timer as the new active timer
			int current_index = m_ActiveHiResTimerIndex;
			m_ActiveHiResTimerIndex = i;

			// get time (do this as late as possible)
			int64 now;
			FRAMEPRO_GET_CLOCK_COUNT(now);

			// pause the current active timer
			if (current_index != -1)
				m_HiResTimers[current_index].m_Duration += now - m_HiResTimerStartTime;
			m_PausedHiResTimerStack.Add(current_index);

			// start the new timer
			m_HiResTimerStartTime = now;
		}

		FRAMEPRO_FORCE_INLINE void StopHiResTimer()
		{
			FRAMEPRO_ASSERT(IsOnTLSThread());

			#if FRAMEPRO_ALLOW_UNPARENTED_HIRES_SCOPES
				if(!m_ScopeDepth)
					return;
			#else
				FRAMEPRO_ASSERT(m_ScopeDepth);	// all hires scopes must have a parent
			#endif

			// get time (do this as early as possible)
			int64 now;
			FRAMEPRO_GET_CLOCK_COUNT(now);

			// get the current active timer
			HiResTimer& timer = m_HiResTimers[m_ActiveHiResTimerIndex];

			// add time and count to active timer
			timer.m_Duration += now - m_HiResTimerStartTime;
			++timer.m_Count;

			// unpause previous timer
			m_ActiveHiResTimerIndex = m_PausedHiResTimerStack.RemoveLast();
			m_HiResTimerStartTime = now;
		}

		FRAMEPRO_FORCE_INLINE bool HasHiResTimers() const
		{
			return m_HiResTimers.GetCount() != 0;
		}

		FRAMEPRO_FORCE_INLINE void SubmitHiResTimers(int64 start_time, int64 end_time)
		{
			FRAMEPRO_ASSERT(IsOnTLSThread());

			if (m_HiResTimers.GetCount() != 0)
				SendHiResTimersScope(start_time, end_time);
		}

		FRAMEPRO_NO_INLINE void SendHiResTimersScope(int64 start_time, int64 end_time);

#ifdef FRAMEPRO_SCOPE_MIN_TIME
		int64 GetScopeMinTime() const { return m_ScopeMinTime; }
#endif

#ifdef FRAMEPRO_WAIT_EVENT_MIN_TIME
		int64 GetWaitEventMinTime() const { return m_WaitEventMinTime; }
#endif

		void SetCustomTimeSpanStat(StringId name, int64 value);

		void SetCustomTimeSpanStat(StringId name, double value);

		void SetCustomTimeSpanStatW(StringId name, int64 value);

		void SetCustomTimeSpanStatW(StringId name, double value);

		void SetCustomStatInfo(const char* p_name, const char* p_graph, const char* p_unit, uint colour);
		
		void SetCustomStatInfo(const wchar_t* p_name, const wchar_t* p_graph, const wchar_t* p_unit, uint colour);

		void SetCustomStatInfo(StringId name, const char* p_graph, const char* p_unit, uint colour);

		void SetCustomStatInfo(StringId name, const wchar_t* p_graph, const wchar_t* p_unit, uint colour);

		void SetCustomStatInfo(StringId name, StringId graph, StringId unit, uint colour);

		int GetAndClearCurrentCallstackId(int64 start_time, int64 end_time);

		void SetCurrentCallstackId(int id);

#if FRAMEPRO_DEBUG || FRAMEPRO_ALLOW_UNPARENTED_HIRES_SCOPES
		void IncScopeDepth() { ++m_ScopeDepth; }
		void DecScopeDepth() { --m_ScopeDepth; FRAMEPRO_ASSERT(m_ScopeDepth >= 0); }
#endif

#if FRAMEPRO_ENABLE_CALLSTACKS
		bool ShouldSendCallstacks() const { return m_SendCallstacks; }

		void SetSendCallstacks(bool b) { m_SendCallstacks = b; }

		CallstackResult GetCallstack();
#endif

	private:
		void Clear();

		void SendString(StringId string_id, const char* p_str, PacketType::Enum packet_type);

		void SendString(StringId string_id, const wchar_t* p_str, PacketType::Enum packet_type);

		void SendSessionInfo(const void* p_data, int size);

		void UpdateStringMemorySize();

		FRAMEPRO_NO_INLINE void FlushCurrentSendBuffer_no_lock();

		bool AllocateCurrentSendBuffer();

		void FreeCurrentSendBuffer();
		
		SendBuffer* AllocateSendBuffer();

		void UpdateSendStringsImmediatelyFlag() { m_SendStringsImmediately = m_RecordingToFile || !m_Interactive; }

		bool AddStringLiteral(const void* p_string)
		{
			bool added = m_LiteralStringSet.Add(p_string);
			m_LiteralStringSetMemorySize = m_LiteralStringSet.GetMemorySize();
			return added;
		}

		ConditionalParentScope* GetConditionalParentScope(const char* p_name);

		ConditionalParentScope* CreateConditionalParentScope(const char* p_name);

		void FlushConditionalChildSendBuffers();

		void PushHiResTimerList();

		void PopHiResTimerList();

		void SendHiResTimerList(HiResTimerList* p_hires_timers, int64 current_time);

		void SendRootHiResTimerList();

		bool HaveSentCustomStatInfo(StringId name);

		void SetHaveSentCustomStatInfo(StringId name);

#if FRAMEPRO_DEBUG
		bool IsOnTLSThread() const { return Platform::GetCurrentThreadId() == m_ThreadId; }
#endif
		template<typename T>
		void DeleteListItems(List<T>& list)
		{
			while (!list.IsEmpty())
			{
				T* p_item = list.RemoveHead();
				Delete(mp_Allocator, p_item);
			}
		}

		//------------------------------------------------------------------------
		// data
	private:

		// keep these together because they are accessed by AddTimeSpan functions, which we need to be fast

#ifdef FRAMEPRO_SCOPE_MIN_TIME
		int64 m_ScopeMinTime;			// read-only after initialisation so no need to be atomic
#endif

#ifdef FRAMEPRO_WAIT_EVENT_MIN_TIME
		int64 m_WaitEventMinTime;			// read-only after initialisation so no need to be atomic
#endif

		RelaxedAtomic<bool> m_Interactive;

		RelaxedAtomic<bool> m_RecordingToFile;
		RelaxedAtomic<bool> m_SendStringsImmediately;

		CriticalSection m_CurrentSendBufferCriticalSection;
		void* mp_CurrentSendBuffer;						// access must be protected with m_CurrentSendBufferCriticalSection
		int m_CurrentSendBufferSize;					// access must be protected with m_CurrentSendBufferCriticalSection
		
		int m_ThreadId;

		int64 m_FirstHiresTimerTime;

		// everything else

		// HiRes timers stuff is only accessed from the tls thread
		Array<HiResTimer> m_HiResTimers;
		Array<int> m_PausedHiResTimerStack;
		int64 m_HiResTimerStartTime;
		int m_ActiveHiResTimerIndex;

		List<SendBuffer> m_SendBufferFreeList;

		FrameProTLS* mp_Next;

		Allocator* mp_Allocator;

		List<SendBuffer> m_SendBufferList;

		PointerSet m_LiteralStringSet;
		RelaxedAtomic<size_t> m_LiteralStringSetMemorySize;

		static std::atomic<long> m_StringCount;
		HashMap<String, StringId> m_StringHashMap;
		HashMap<WString, StringId> m_WStringHashMap;

		Array<bool> m_InitialisedCustomStats;

		Buffer m_SessionInfoBuffer;
		CriticalSection m_SessionInfoBufferLock;
		RelaxedAtomic<size_t> m_SessionInfoBufferMemorySize;

		CriticalSection m_CriticalSection;

		std::atomic<bool> m_Connected;

		IncrementingBlockAllocator m_StringAllocator;

		RelaxedAtomic<size_t> m_SendBufferMemorySize;
		RelaxedAtomic<size_t> m_StringMemorySize;

		static const int m_SendBufferCapacity = 32768 - 16;

		int64 m_ClockFrequency;

		RelaxedAtomic<bool> m_ShuttingDown;

		// conditional parent scope
		CriticalSection m_ConditionalParentScopeListCritSec;
		List<ConditionalParentScope> m_ConditionalParentScopeList;
		ConditionalParentScope* mp_CurrentConditionalParentScope;

		char m_FalseSharingSpacerBuffer[128];		// separate TLS classes to avoid false sharing

		int m_CurrentCallstackId;
		int64 m_CurrentCallstackIdTime;

#if FRAMEPRO_ENABLE_CALLSTACKS
		StackTrace m_StackTrace;
		bool m_SendCallstacks;
#endif

#if FRAMEPRO_DEBUG || FRAMEPRO_ALLOW_UNPARENTED_HIRES_SCOPES
		int m_ScopeDepth;
#endif
	};
}


//------------------------------------------------------------------------
//
// Event.h
//

//------------------------------------------------------------------------
namespace FramePro
{
	//--------------------------------------------------------------------
	class Event
	{
	public:
		//--------------------------------------------------------------------
		Event(bool initial_state, bool auto_reset)
		{
			Platform::CreateEventX(m_OSEventMem, sizeof(m_OSEventMem), initial_state, auto_reset);
		}

		//--------------------------------------------------------------------
		~Event()
		{
			Platform::DestroyEvent(m_OSEventMem);
		}

		//--------------------------------------------------------------------
		void Set() const
		{
			Platform::SetEvent(m_OSEventMem);
		}

		//--------------------------------------------------------------------
		void Reset()
		{
			Platform::ResetEvent(m_OSEventMem);
		}

		//--------------------------------------------------------------------
		int Wait(int timeout=-1) const
		{
			return Platform::WaitEvent(m_OSEventMem, timeout);
		}

		//------------------------------------------------------------------------
		// data
	private:
		static const int m_OSEventMemMaxSize = 96;
		mutable char m_OSEventMem[m_OSEventMemMaxSize];
	} FRAMEPRO_ALIGN_STRUCT(16);
}


//------------------------------------------------------------------------
//
// Thread.h
//

//------------------------------------------------------------------------
namespace FramePro
{
	//------------------------------------------------------------------------
	class Thread
	{
	public:
		Thread();

		~Thread();

		bool CreateThread(ThreadMain p_thread_main, void* p_param, Allocator* p_allocator);

		bool IsAlive() const { return m_Alive; }

		void SetPriority(int priority);

		void SetAffinity(int affinity);

		void WaitForThreadToTerminate(int timeout);

	private:
		static int ThreadMain(void* p_context);

		//------------------------------------------------------------------------
		// data
	private:
		static const int m_OSThreadMaxSize = 16;
		char m_OSThread[m_OSThreadMaxSize];
		
		bool m_Created;

		bool m_Alive;

		FramePro::ThreadMain mp_ThreadMain;
		void* mp_Param;

		Event m_ThreadTerminatedEvent;
	};
}


//------------------------------------------------------------------------
//
// File.h
//

//------------------------------------------------------------------------
namespace FramePro
{
	//------------------------------------------------------------------------
	class Allocator;

	//------------------------------------------------------------------------
	class File
	{
	public:
		//------------------------------------------------------------------------
		File()
		:	m_Opened(false)
		{
		}

		//------------------------------------------------------------------------
		void SetAllocator(Allocator* p_allocator)
		{
			m_Filename.SetAllocator(p_allocator);
		}

		//------------------------------------------------------------------------
		bool OpenForRead(const char* p_filename)
		{
			FRAMEPRO_ASSERT(!m_Opened);
			m_Filename = p_filename;
			m_Opened = Platform::OpenFileForRead(m_OSFile, m_OSFileMaxSize, p_filename);
			return m_Opened;
		}

		//------------------------------------------------------------------------
		bool OpenForRead(const wchar_t* p_filename)
		{
			FRAMEPRO_ASSERT(!m_Opened);
			m_Filename = p_filename;
			m_Opened = Platform::OpenFileForRead(m_OSFile, m_OSFileMaxSize, p_filename);
			return m_Opened;
		}

		//------------------------------------------------------------------------
		bool OpenForWrite(const char* p_filename)
		{
			FRAMEPRO_ASSERT(!m_Opened);
			m_Filename = p_filename;
			m_Opened = Platform::OpenFileForWrite(m_OSFile, m_OSFileMaxSize, p_filename);
			return m_Opened;
		}

		//------------------------------------------------------------------------
		bool OpenForWrite(const wchar_t* p_filename)
		{
			FRAMEPRO_ASSERT(!m_Opened);
			m_Filename = p_filename;
			m_Opened = Platform::OpenFileForWrite(m_OSFile, m_OSFileMaxSize, p_filename);
			return m_Opened;
		}

		//------------------------------------------------------------------------
		void Close()
		{
			FRAMEPRO_ASSERT(m_Opened);
			m_Filename.Clear();
			Platform::CloseFile(m_OSFile);
			m_Opened = false;
		}

		//------------------------------------------------------------------------
		void Read(void* p_data, size_t size)
		{
			FRAMEPRO_ASSERT(m_Opened);
			Platform::ReadFromFile(m_OSFile, p_data, size);
		}

		//------------------------------------------------------------------------
		void Write(const void* p_data, size_t size)
		{
			FRAMEPRO_ASSERT(m_Opened);
			Platform::WriteToFile(m_OSFile, p_data, size);
		}

		//------------------------------------------------------------------------
		bool IsOpened() const
		{
			return m_Opened;
		}

		//------------------------------------------------------------------------
		size_t GetSize() const
		{
			FRAMEPRO_ASSERT(m_Opened);
			return Platform::GetFileSize(m_OSFile);
		}

		//------------------------------------------------------------------------
		const DynamicWString& GetFilename() const
		{
			return m_Filename;
		}

		//------------------------------------------------------------------------
		// data
	private:
		static const int m_OSFileMaxSize = 16;
		char m_OSFile[m_OSFileMaxSize];

		bool m_Opened;

		DynamicWString m_Filename;
	};
}


//------------------------------------------------------------------------
//
// FrameProSession.h
//

//------------------------------------------------------------------------
namespace FramePro
{
	//------------------------------------------------------------------------
	class FrameProTLS;
	class Allocator;
	struct ThreadCustomStats;
	class SendBuffer;

	//------------------------------------------------------------------------
	class FrameProSession
	{
	public:
		FrameProSession();

		~FrameProSession();
		
		void BlockSockets();

		void UnblockSockets();

		void FrameStart();

		void Shutdown();

		int64 GetClockFrequency();

		bool AddFrameProTLS(FrameProTLS* p_framepro_tls);

		void RemoveFrameProTLS(FrameProTLS* p_framepro_tls);

		void SetPort(int port);
		
		void SetAllocator(Allocator* p_allocator);

		Allocator* GetAllocator()
		{
			if(!m_Allocator.HasAllocator())
				CreateDefaultAllocator();

			return &m_Allocator;
		}

		void SetThreadName(const char* p_name);

		void SetThreadName(int thread_id, const char* p_name);

		void StartRecording(const char* p_filename, bool context_switches, bool callstacks, int64 max_file_size);

		void StartRecording(const wchar_t* p_filename, bool context_switches, bool callstacks, int64 max_file_size);

		void StopRecording();

		void RegisterConnectionChangedCallback(ConnectionChangedCallback p_callback, void* p_context);

		void UnregisterConnectionChangedCallback(ConnectionChangedCallback p_callback);

		void SetThreadPriority(int priority);
		
		void SetThreadAffinity(int affinity);

		void SendSessionInfo(const char* p_name, const char* p_value);

		void SendSessionInfo(const wchar_t* p_name, const wchar_t* p_value);

		void AddCustomStatTimer(CustomStatTimer* p_timer);

		bool CallConditionalParentScopeCallback(ConditionalParentScopeCallback p_callback, const char* p_name, int64 start_time, int64 end_time);

		void SetConditionalScopeMinTimeInMicroseconds(int64 value);

		void SetScopeColour(StringId name, uint colour);

		void SetCustomStatGraph(StringId name, StringId graph);

		void SetCustomStatUnit(StringId name, StringId unit);

		void SetCustomStatColour(StringId name, uint colour);

		void SendScopeCallstack();

	private:
		void Initialise(FrameProTLS* p_framepro_tls);

		void StopRecording_NoLock();

		void SendSessionInfo(StringId name, StringId value);

		void CreateDefaultAllocator();

		void SetAllocatorInternal(Allocator* p_allocator);

		bool InitialiseConnection(FrameProTLS* p_framepro_tls);

		FRAMEPRO_FORCE_INLINE void CalculateTimerFrequency();

		void SetConnected(bool value);

		bool Send(const void* p_data, int size);

		void SendFrameBuffer();

		static int StaticSendThreadMain(void*);

		int SendThreadMain();

		void Disconnect(bool wait_for_threads_to_exit=true);

		void Disconnect_NoLock(bool wait_for_threads_to_exit=true);

		void SendRecordedDataAndDisconnect();

		void SendHeartbeatInfo(FrameProTLS* p_framepro_tls);

		void SendImmediate(void* p_data, int size, FrameProTLS* p_framepro_tls);

		bool HasSetThreadName(int thread_id) const;

		void OnConnectionChanged(bool connected, const DynamicWString& filename) const;

		int GetConnectionChangedCallbackIndex(ConnectionChangedCallback p_callback);

		size_t GetMemoryUsage() const;

		void CreateSendThread();

		static void ContextSwitchCallback_Static(const ContextSwitch& context_switch, void* p_param);

		void ContextSwitchCallback(const ContextSwitch& context_switch);

		void StartRecordingContextSitches();

		void FlushCustomStatTimers(FrameProTLS* p_framepro_tls);

		void ClearCustomStatTimers();

		void SendExtraModuleInfo(int64 ModuleBase, FrameProTLS* p_framepro_tls);

		#if FRAMEPRO_SOCKETS_ENABLED
			bool InitialiseFileCache();

			static int StaticConnectThreadMain(void*);

			int ConnectThreadMain();

			static int StaticReceiveThreadMain(void*);

			int OnReceiveThreadExit();

			int ReceiveThreadMain();

			void OpenListenSocket();

			void StartConnectThread();

			void CreateReceiveThread();
		#endif

		void SendOnMainThread(void* p_src, int size);

		template<typename T>
		void SendOnMainThread(T& packet)
		{
			SendOnMainThread(&packet, sizeof(packet));
		}

		#if FRAMEPRO_ENABLE_CALLSTACKS
			void SetCallstacksEnabled(bool enabled);

			void SendModules();
		#endif

		void SendScopeColours();

		void SendCustomStatGraphs();
		
		void SendCustomStatUnits();

		void SendCustomStatColours();

		//------------------------------------------------------------------------
		// data
	private:
		static FrameProSession* mp_Inst;		// just used for debugging

		mutable CriticalSection m_CriticalSection;

		char m_Port[8];

		CheckedAllocator m_Allocator;
		Allocator* mp_CreatedAllocator;

		bool m_Initialised;

		std::atomic<bool> m_InitialiseConnectionNextFrame;

		std::atomic<bool> m_StartContextSwitchRecording;

		#if FRAMEPRO_ENABLE_CALLSTACKS
			std::atomic<bool> m_StartRecordingCallstacks;
		#endif
			
		int64 m_ClockFrequency;

		mutable CriticalSection m_TLSListCriticalSection;
		List<FrameProTLS> m_FrameProTLSList;

		int m_MainThreadId;

		Thread m_SendThread;
		Event m_SendThreadStarted;
		Event m_SendReady;
		Event m_SendComplete;

		Thread m_ReceiveThread;
		Event m_ReceiveThreadTerminatedEvent;

		CriticalSection m_SendFrameBufferCriticalSection;

		RelaxedAtomic<bool> m_Interactive;
		File m_NonInteractiveRecordingFile;
		int64 m_NonInteractiveRecordingFileSize;

		int64 m_LastSessionInfoSendTime;

		Array<int> m_NamedThreads;

		File m_RecordingFile;
		int64 m_RecordingFileSize;
		int64 m_MaxRecordingFileSize;

		bool m_ThreadPrioritySet;
		int m_ThreadPriority;
		bool m_ThreadAffinitySet;
		int m_ThreadAffinity;

		#if FRAMEPRO_SOCKETS_ENABLED
			Thread m_ConnectThread;
			Socket m_ListenSocket;
			Socket m_ClientSocket;
		#endif

		std::atomic<bool> m_SendThreadExit;
		Event m_SendThreadFinished;

		bool m_SocketsBlocked;

		struct ConnectionChangedcallbackInfo
		{
			ConnectionChangedCallback mp_Callback;
			void* mp_Context;
		};
		mutable CriticalSection m_ConnectionChangedCriticalSection;
		Array<ConnectionChangedcallbackInfo> m_Connectionchangedcallbacks;

		Array<int> m_ProcessIds;

		Buffer m_MainThreadSendBuffer[2];
		int m_MainThreadSendBufferIndex;
		CriticalSection m_MainThreadSendBufferLock;

		Array<RequestStringLiteralPacket> m_StringRequestPackets;
		CriticalSection m_StringRequestPacketsLock;

		CustomStatTimer* m_CustomStatTimers;

		Array<ModulePacket*> m_ModulePackets;

		void* mp_ContextSwitchRecorder;

		struct ScopeColour
		{
			StringId m_Name;
			uint m_Colour;
		};
		Array<ScopeColour> m_ScopeColours;
		CriticalSection m_ScopeColoursLock;

		struct CustomStatInfo
		{
			StringId m_Name;
			StringId m_Value;
		};

		Array<CustomStatInfo> m_CustomStatGraphs;
		Array<CustomStatInfo> m_CustomStatUnits;
		Array<ScopeColour> m_CustomStatColours;
		CriticalSection m_CustomStatInfoLock;

		#if FRAMEPRO_ENABLE_CALLSTACKS
			bool m_SendModules;
		#endif
	};
}


//------------------------------------------------------------------------
//
// FramePro.cpp
//

//------------------------------------------------------------------------
namespace FramePro
{
	//------------------------------------------------------------------------
	RelaxedAtomic<bool> g_Connected = false;

	RelaxedAtomic<unsigned int> g_ConditionalScopeMinTime = UINT_MAX;

	//------------------------------------------------------------------------
	FrameProSession& GetFrameProSession()
	{
		static FrameProSession session;
		return session;
	}
		
	//------------------------------------------------------------------------
	#if FRAMEPRO_USE_TLS_SLOTS
		uint GetFrameProTLSSlot()
		{
			static uint slot = Platform::AllocateTLSSlot();
			return slot;
		}
	#endif

	//------------------------------------------------------------------------
	FRAMEPRO_NO_INLINE FrameProTLS* CreateFrameProTLS()
	{
		FrameProSession& framepro_session = GetFrameProSession();

		Allocator* p_allocator = framepro_session.GetAllocator();

		FrameProTLS* p_framepro_tls = (FrameProTLS*)p_allocator->Alloc(sizeof(FrameProTLS));
		
		if (!p_framepro_tls)
			return nullptr;

		new (p_framepro_tls)FrameProTLS(p_allocator, framepro_session.GetClockFrequency());

		if (!framepro_session.AddFrameProTLS(p_framepro_tls))
			return NULL;

		#if FRAMEPRO_USE_TLS_SLOTS
			int slot = GetFrameProTLSSlot();
		#else
			int slot = 0;
		#endif

		Platform::SetTLSValue(slot, p_framepro_tls);

		return p_framepro_tls;
	}

	//------------------------------------------------------------------------
	FRAMEPRO_NO_INLINE void DestroyFrameProTLS(FrameProTLS* p_framepro_tls)
	{
		FrameProSession& framepro_session = GetFrameProSession();

		framepro_session.RemoveFrameProTLS(p_framepro_tls);

		p_framepro_tls->~FrameProTLS();

		framepro_session.GetAllocator()->Free(p_framepro_tls);
	}

	//------------------------------------------------------------------------
	void SendWaitEventPacket(
		int64 event_id,
		int64 time,
		int core,
		PacketType::Enum packet_type)
	{
		if (!g_Connected)
			return;

		FrameProTLS* p_framepro_tls = GetFrameProTLS();
		if (!p_framepro_tls)
			return;

		CriticalSectionScope lock(p_framepro_tls->GetCurrentSendBufferCriticalSection());

		WaitEventPacket* p_packet = p_framepro_tls->AllocateSpaceInBuffer<WaitEventPacket>();

		p_packet->m_PacketType = packet_type;
		p_packet->m_Thread = p_framepro_tls->GetThreadId();
		p_packet->m_Core = core;
		p_packet->m_EventId = event_id;
		p_packet->m_Time = time;
	}
}

//------------------------------------------------------------------------
void FramePro::SetAllocator(Allocator* p_allocator)
{
	GetFrameProSession().SetAllocator(p_allocator);
}

//------------------------------------------------------------------------
void FramePro::DebugBreak()
{
	Platform::DebugBreak();
}

//------------------------------------------------------------------------
void FramePro::Shutdown()
{
	GetFrameProSession().Shutdown();
}

//------------------------------------------------------------------------
void FramePro::FrameStart()
{
	GetFrameProSession().FrameStart();
}

//------------------------------------------------------------------------
void FramePro::RegisterConnectionChangedCallback(ConnectionChangedCallback p_callback, void* p_context)
{
	GetFrameProSession().RegisterConnectionChangedCallback(p_callback, p_context);
}

//------------------------------------------------------------------------
void FramePro::UnregisterConnectionChangedcallback(ConnectionChangedCallback p_callback)
{
	GetFrameProSession().UnregisterConnectionChangedCallback(p_callback);
}

//------------------------------------------------------------------------
void FramePro::AddTimeSpan(const char* p_name_and_source_info, int64 start_time, int64 end_time)
{
	FrameProTLS* p_framepro_tls = GetFrameProTLS();
	if (!p_framepro_tls)
		return;

#ifdef FRAMEPRO_SCOPE_MIN_TIME
	if (end_time - start_time < p_framepro_tls->GetScopeMinTime())
		return;
#endif

	p_framepro_tls->SubmitHiResTimers(start_time, end_time);

	// if we are connecting to FramePro, FramePro will ask for the string value later, otherwise we need to send it now
	if(p_framepro_tls->SendStringsImmediately())
		p_framepro_tls->SendString(p_name_and_source_info, PacketType::NameAndSourceInfoPacket);

#if FRAMEPRO_ENABLE_CALLSTACKS
	int current_callstack_id = p_framepro_tls->GetAndClearCurrentCallstackId(start_time, end_time);

	if (current_callstack_id != -1 || p_framepro_tls->ShouldSendCallstacks())
	{
		CallstackResult callstack_result;

		if (current_callstack_id == -1)
		{
			callstack_result = p_framepro_tls->GetCallstack();
			current_callstack_id = callstack_result.mp_Callstack->m_ID;
		}
		else
		{
			p_framepro_tls->SetCurrentCallstackId(-1);
		}

		int send_size = sizeof(TimeSpanPacket) + sizeof(TimeSpanCallstackPacket);
		if (callstack_result.m_IsNew)
			send_size += callstack_result.mp_Callstack->m_Size * sizeof(uint64);

		{
			CriticalSectionScope lock(p_framepro_tls->GetCurrentSendBufferCriticalSection());

			TimeSpanPacket* p_packet = (TimeSpanPacket*)p_framepro_tls->AllocateSpaceInBuffer(send_size);

			p_packet->m_PacketType_AndCore = PacketType::TimeSpanWithCallstack | (FramePro::Platform::GetCore() << 16);
			p_packet->m_ThreadId = p_framepro_tls->GetThreadId();
			p_packet->m_NameAndSourceInfo = (StringId)p_name_and_source_info;
			p_packet->m_StartTime = start_time;
			p_packet->m_EndTime = end_time;

			TimeSpanCallstackPacket* p_callstack_packet = (TimeSpanCallstackPacket*)(p_packet + 1);
			p_callstack_packet->m_CallstackId = current_callstack_id;
			p_callstack_packet->m_CallstackSize = 0;

			if (callstack_result.m_IsNew)
			{
				p_callstack_packet->m_CallstackSize = callstack_result.mp_Callstack->m_Size;
				memcpy(
					(char*)(p_callstack_packet + 1),
					callstack_result.mp_Callstack->mp_Stack,
					callstack_result.mp_Callstack->m_Size * sizeof(uint64));
			}
		}
	}
	else
#endif
	{
		CriticalSectionScope lock(p_framepro_tls->GetCurrentSendBufferCriticalSection());

		TimeSpanPacket* p_packet = p_framepro_tls->AllocateSpaceInBuffer<TimeSpanPacket>();

		p_packet->m_PacketType_AndCore = PacketType::TimeSpan | (FramePro::Platform::GetCore() << 16);
		p_packet->m_ThreadId = p_framepro_tls->GetThreadId();
		p_packet->m_NameAndSourceInfo = (StringId)p_name_and_source_info;
		p_packet->m_StartTime = start_time;
		p_packet->m_EndTime = end_time;
	}
}

//------------------------------------------------------------------------
void FramePro::AddTimeSpan(const wchar_t* p_name_and_source_info, int64 start_time, int64 end_time)
{
	FRAMEPRO_ASSERT(start_time <= end_time);

	FrameProTLS* p_framepro_tls = GetFrameProTLS();
	if (!p_framepro_tls)
		return;

	p_framepro_tls->SubmitHiResTimers(start_time, end_time);

#ifdef FRAMEPRO_SCOPE_MIN_TIME
	if (end_time - start_time < p_framepro_tls->GetScopeMinTime())
		return;
#endif

	// if we are connecting to FramePro, FramePro will ask for the string value later, otherwise we need to send it now
	if(p_framepro_tls->SendStringsImmediately())
		p_framepro_tls->SendString(p_name_and_source_info, PacketType::NameAndSourceInfoPacketW);

#if FRAMEPRO_ENABLE_CALLSTACKS
	int current_callstack_id = p_framepro_tls->GetAndClearCurrentCallstackId(start_time, end_time);

	if (current_callstack_id != -1 || p_framepro_tls->ShouldSendCallstacks())
	{
		CallstackResult callstack_result;

		if (current_callstack_id == -1)
		{
			callstack_result = p_framepro_tls->GetCallstack();
			current_callstack_id = callstack_result.mp_Callstack->m_ID;
		}
		else
		{
			p_framepro_tls->SetCurrentCallstackId(-1);
		}

		int send_size = sizeof(TimeSpanPacket) + sizeof(TimeSpanCallstackPacket);
		if (callstack_result.m_IsNew)
			send_size += callstack_result.mp_Callstack->m_Size * sizeof(uint64);

		{
			CriticalSectionScope lock(p_framepro_tls->GetCurrentSendBufferCriticalSection());

			TimeSpanPacket* p_packet = (TimeSpanPacket*)p_framepro_tls->AllocateSpaceInBuffer(send_size);

			p_packet->m_PacketType_AndCore = PacketType::TimeSpanWWithCallstack | (FramePro::Platform::GetCore() << 16);
			p_packet->m_ThreadId = p_framepro_tls->GetThreadId();
			p_packet->m_NameAndSourceInfo = (StringId)p_name_and_source_info;
			p_packet->m_StartTime = start_time;
			p_packet->m_EndTime = end_time;

			TimeSpanCallstackPacket* p_callstack_packet = (TimeSpanCallstackPacket*)(p_packet + 1);
			p_callstack_packet->m_CallstackId = current_callstack_id;
			p_callstack_packet->m_CallstackSize = 0;

			if (callstack_result.m_IsNew)
			{
				p_callstack_packet->m_CallstackSize = callstack_result.mp_Callstack->m_Size;
				memcpy(
					(char*)(p_callstack_packet + 1),
					callstack_result.mp_Callstack->mp_Stack,
					callstack_result.mp_Callstack->m_Size * sizeof(uint64));
			}
		}
	}
	else
#endif
	{
		CriticalSectionScope lock(p_framepro_tls->GetCurrentSendBufferCriticalSection());

		TimeSpanPacket* p_packet = p_framepro_tls->AllocateSpaceInBuffer<TimeSpanPacket>();

		p_packet->m_PacketType_AndCore = PacketType::TimeSpanW | (FramePro::Platform::GetCore() << 16);
		p_packet->m_ThreadId = p_framepro_tls->GetThreadId();
		p_packet->m_NameAndSourceInfo = (StringId)p_name_and_source_info;
		p_packet->m_StartTime = start_time;
		p_packet->m_EndTime = end_time;
	}
}

//------------------------------------------------------------------------
UE_AUTORTFM_ALWAYS_OPEN
void FramePro::AddTimeSpan(StringId name, const char* p_source_info, int64 start_time, int64 end_time)
{
	FRAMEPRO_ASSERT(start_time <= end_time);

	FrameProTLS* p_framepro_tls = GetFrameProTLS();
	if (!p_framepro_tls)
		return;

	p_framepro_tls->SubmitHiResTimers(start_time, end_time);

#ifdef FRAMEPRO_SCOPE_MIN_TIME
	if (end_time - start_time < p_framepro_tls->GetScopeMinTime())
		return;
#endif

	// if we are connecting to FramePro, FramePro will ask for the string value later, otherwise we need to send it now
	if(p_framepro_tls->SendStringsImmediately())
		p_framepro_tls->SendString(p_source_info, PacketType::SourceInfoPacket);

#if FRAMEPRO_ENABLE_CALLSTACKS
	int current_callstack_id = p_framepro_tls->GetAndClearCurrentCallstackId(start_time, end_time);

	if (current_callstack_id != -1 || p_framepro_tls->ShouldSendCallstacks())
	{
		CallstackResult callstack_result;

		if (current_callstack_id == -1)
		{
			callstack_result = p_framepro_tls->GetCallstack();
			current_callstack_id = callstack_result.mp_Callstack->m_ID;
		}
		else
		{
			p_framepro_tls->SetCurrentCallstackId(-1);
		}

		int send_size = sizeof(NamedTimeSpanPacket) + sizeof(TimeSpanCallstackPacket);
		if (callstack_result.m_IsNew)
			send_size += callstack_result.mp_Callstack->m_Size * sizeof(uint64);

		{
			CriticalSectionScope lock(p_framepro_tls->GetCurrentSendBufferCriticalSection());

			NamedTimeSpanPacket* p_packet = (NamedTimeSpanPacket*)p_framepro_tls->AllocateSpaceInBuffer(send_size);

			p_packet->m_PacketType_AndCore = PacketType::NamedTimeSpanWithCallstack | (FramePro::Platform::GetCore() << 16);
			p_packet->m_ThreadId = p_framepro_tls->GetThreadId();
			p_packet->m_Name = name;
			p_packet->m_SourceInfo = (StringId)p_source_info;
			p_packet->m_StartTime = start_time;
			p_packet->m_EndTime = end_time;

			TimeSpanCallstackPacket* p_callstack_packet = (TimeSpanCallstackPacket*)(p_packet + 1);
			p_callstack_packet->m_CallstackId = current_callstack_id;
			p_callstack_packet->m_CallstackSize = 0;

			if (callstack_result.m_IsNew)
			{
				p_callstack_packet->m_CallstackSize = callstack_result.mp_Callstack->m_Size;
				memcpy(
					(char*)(p_callstack_packet + 1),
					callstack_result.mp_Callstack->mp_Stack,
					callstack_result.mp_Callstack->m_Size * sizeof(uint64));
			}
		}
	}
	else
#endif
	{
		CriticalSectionScope lock(p_framepro_tls->GetCurrentSendBufferCriticalSection());

		NamedTimeSpanPacket* p_packet = p_framepro_tls->AllocateSpaceInBuffer<NamedTimeSpanPacket>();

		p_packet->m_PacketType_AndCore = PacketType::NamedTimeSpan | (FramePro::Platform::GetCore() << 16);
		p_packet->m_ThreadId = p_framepro_tls->GetThreadId();
		p_packet->m_Name = name;
		p_packet->m_SourceInfo = (StringId)p_source_info;
		p_packet->m_StartTime = start_time;
		p_packet->m_EndTime = end_time;
	}
}

//------------------------------------------------------------------------
void FramePro::AddTimeSpan(StringId name, const char* p_source_info, int64 start_time, int64 end_time, int thread_id, int core)
{
	FRAMEPRO_ASSERT(start_time <= end_time);

	FrameProTLS* p_framepro_tls = GetFrameProTLS();
	if (!p_framepro_tls)
		return;

	p_framepro_tls->SubmitHiResTimers(start_time, end_time);

#ifdef FRAMEPRO_SCOPE_MIN_TIME
	if (end_time - start_time < p_framepro_tls->GetScopeMinTime())
		return;
#endif

	// if we are connecting to FramePro, FramePro will ask for the string value later, otherwise we need to send it now
	if(p_framepro_tls->SendStringsImmediately())
		p_framepro_tls->SendString(p_source_info, PacketType::SourceInfoPacket);

#if FRAMEPRO_ENABLE_CALLSTACKS
	int current_callstack_id = p_framepro_tls->GetAndClearCurrentCallstackId(start_time, end_time);

	if (current_callstack_id != -1 || p_framepro_tls->ShouldSendCallstacks())
	{
		CallstackResult callstack_result;

		if (current_callstack_id == -1)
		{
			callstack_result = p_framepro_tls->GetCallstack();
			current_callstack_id = callstack_result.mp_Callstack->m_ID;
		}
		else
		{
			p_framepro_tls->SetCurrentCallstackId(-1);
		}

		int send_size = sizeof(NamedTimeSpanPacket) + sizeof(TimeSpanCallstackPacket);
		if (callstack_result.m_IsNew)
			send_size += callstack_result.mp_Callstack->m_Size * sizeof(uint64);

		{
			CriticalSectionScope lock(p_framepro_tls->GetCurrentSendBufferCriticalSection());

			NamedTimeSpanPacket* p_packet = (NamedTimeSpanPacket*)p_framepro_tls->AllocateSpaceInBuffer(send_size);

			p_packet->m_PacketType_AndCore = PacketType::NamedTimeSpanWithCallstack | (core << 16);
			p_packet->m_ThreadId = thread_id;
			p_packet->m_Name = name;
			p_packet->m_SourceInfo = (StringId)p_source_info;
			p_packet->m_StartTime = start_time;
			p_packet->m_EndTime = end_time;

			TimeSpanCallstackPacket* p_callstack_packet = (TimeSpanCallstackPacket*)(p_packet + 1);
			p_callstack_packet->m_CallstackId = current_callstack_id;
			p_callstack_packet->m_CallstackSize = 0;

			if (callstack_result.m_IsNew)
			{
				p_callstack_packet->m_CallstackSize = callstack_result.mp_Callstack->m_Size;
				memcpy(
					(char*)(p_callstack_packet + 1),
					callstack_result.mp_Callstack->mp_Stack,
					callstack_result.mp_Callstack->m_Size * sizeof(uint64));
			}
		}
	}
	else
#endif
	{
		CriticalSectionScope lock(p_framepro_tls->GetCurrentSendBufferCriticalSection());

		NamedTimeSpanPacket* p_packet = p_framepro_tls->AllocateSpaceInBuffer<NamedTimeSpanPacket>();

		p_packet->m_PacketType_AndCore = PacketType::NamedTimeSpan | (core << 16);
		p_packet->m_ThreadId = thread_id;
		p_packet->m_Name = name;
		p_packet->m_SourceInfo = (StringId)p_source_info;
		p_packet->m_StartTime = start_time;
		p_packet->m_EndTime = end_time;
	}
}

//------------------------------------------------------------------------
// p_name is a string literal
void FramePro::AddTimeSpan(const char* p_name, const char* p_source_info, int64 start_time, int64 end_time)
{
	FrameProTLS* p_framepro_tls = GetFrameProTLS();
	if (!p_framepro_tls)
		return;

	p_framepro_tls->SubmitHiResTimers(start_time, end_time);

#ifdef FRAMEPRO_SCOPE_MIN_TIME
	if (end_time - start_time < p_framepro_tls->GetScopeMinTime())
		return;
#endif

	// if we are connecting to FramePro, FramePro will ask for the string value later, otherwise we need to send it now
	if(p_framepro_tls->SendStringsImmediately())
	{
		p_framepro_tls->SendString(p_name, PacketType::StringLiteralTimerNamePacket);
		p_framepro_tls->SendString(p_source_info, PacketType::SourceInfoPacket);
	}

#if FRAMEPRO_ENABLE_CALLSTACKS
	int current_callstack_id = p_framepro_tls->GetAndClearCurrentCallstackId(start_time, end_time);

	if (current_callstack_id != -1 || p_framepro_tls->ShouldSendCallstacks())
	{
		CallstackResult callstack_result;

		if (current_callstack_id == -1)
		{
			callstack_result = p_framepro_tls->GetCallstack();
			current_callstack_id = callstack_result.mp_Callstack->m_ID;
		}
		else
		{
			p_framepro_tls->SetCurrentCallstackId(-1);
		}

		int send_size = sizeof(NamedTimeSpanPacket) + sizeof(TimeSpanCallstackPacket);
		if (callstack_result.m_IsNew)
			send_size += callstack_result.mp_Callstack->m_Size * sizeof(uint64);

		{
			CriticalSectionScope lock(p_framepro_tls->GetCurrentSendBufferCriticalSection());

			NamedTimeSpanPacket* p_packet = (NamedTimeSpanPacket*)p_framepro_tls->AllocateSpaceInBuffer(send_size);

			p_packet->m_PacketType_AndCore = PacketType::StringLiteralNamedTimeSpanWithCallstack | (FramePro::Platform::GetCore() << 16);
			p_packet->m_ThreadId = p_framepro_tls->GetThreadId();
			p_packet->m_Name = (StringId)p_name;
			p_packet->m_SourceInfo = (StringId)p_source_info;
			p_packet->m_StartTime = start_time;
			p_packet->m_EndTime = end_time;

			TimeSpanCallstackPacket* p_callstack_packet = (TimeSpanCallstackPacket*)(p_packet + 1);
			p_callstack_packet->m_CallstackId = current_callstack_id;
			p_callstack_packet->m_CallstackSize = 0;

			if (callstack_result.m_IsNew)
			{
				p_callstack_packet->m_CallstackSize = callstack_result.mp_Callstack->m_Size;
				memcpy(
					(char*)(p_callstack_packet + 1),
					callstack_result.mp_Callstack->mp_Stack,
					callstack_result.mp_Callstack->m_Size * sizeof(uint64));
			}
		}
	}
	else
#endif
	{
		CriticalSectionScope lock(p_framepro_tls->GetCurrentSendBufferCriticalSection());

		NamedTimeSpanPacket* p_packet = p_framepro_tls->AllocateSpaceInBuffer<NamedTimeSpanPacket>();

		p_packet->m_PacketType_AndCore = PacketType::StringLiteralNamedTimeSpan | (FramePro::Platform::GetCore() << 16);
		p_packet->m_ThreadId = p_framepro_tls->GetThreadId();
		p_packet->m_Name = (StringId)p_name;
		p_packet->m_SourceInfo = (StringId)p_source_info;
		p_packet->m_StartTime = start_time;
		p_packet->m_EndTime = end_time;
	}
}

//------------------------------------------------------------------------
void FramePro::AddCustomStat(const char* p_name, int value, const char* p_graph, const char* p_unit, uint colour)
{
	AddCustomStat(p_name, (int64)value, p_graph, p_unit, colour);
}

//------------------------------------------------------------------------
void FramePro::AddCustomStat(const char* p_name, int64 value, const char* p_graph, const char* p_unit, uint colour)
{
	FrameProTLS* p_framepro_tls = GetFrameProTLS();
	if (!p_framepro_tls)
		return;

	// if we are connecting to FramePro, FramePro will ask for the string value later, otherwise we need to send it now
	if (p_framepro_tls->SendStringsImmediately())
		p_framepro_tls->SendString(p_name, PacketType::StringPacket);

	p_framepro_tls->SetCustomStatInfo(p_name, p_graph, p_unit, colour);	// only sends first time

	CriticalSectionScope lock(p_framepro_tls->GetCurrentSendBufferCriticalSection());

	CustomStatPacketInt64* p_packet = p_framepro_tls->AllocateSpaceInBuffer<CustomStatPacketInt64>();

	CustomStatValueType::Enum value_type = CustomStatValueType::Int64;

	p_packet->m_PacketTypeAndValueType = PacketType::CustomStatPacket | (value_type << 16);
	p_packet->m_Count = 1;
	p_packet->m_Name = (StringId)p_name;
	p_packet->m_Value = value;
	FRAMEPRO_GET_CLOCK_COUNT(p_packet->m_Time);
}

//------------------------------------------------------------------------
void FramePro::AddCustomStat(const char* p_name, float value, const char* p_graph, const char* p_unit, uint colour)
{
	AddCustomStat(p_name, (double)value, p_graph, p_unit, colour);
}

//------------------------------------------------------------------------
void FramePro::AddCustomStat(const char* p_name, double value, const char* p_graph, const char* p_unit, uint colour)
{
	FrameProTLS* p_framepro_tls = GetFrameProTLS();
	if (!p_framepro_tls)
		return;

	// if we are connecting to FramePro, FramePro will ask for the string value later, otherwise we need to send it now
	if (p_framepro_tls->SendStringsImmediately())
		p_framepro_tls->SendString(p_name, PacketType::StringPacket);

	p_framepro_tls->SetCustomStatInfo(p_name, p_graph, p_unit, colour);	// only sends first time

	CriticalSectionScope lock(p_framepro_tls->GetCurrentSendBufferCriticalSection());

	CustomStatPacketDouble* p_packet = p_framepro_tls->AllocateSpaceInBuffer<CustomStatPacketDouble>();

	CustomStatValueType::Enum value_type = CustomStatValueType::Double;

	p_packet->m_PacketTypeAndValueType = PacketType::CustomStatPacket | (value_type << 16);
	p_packet->m_Count = 1;
	p_packet->m_Name = (StringId)p_name;
	p_packet->m_Value = value;
	FRAMEPRO_GET_CLOCK_COUNT(p_packet->m_Time);
}

//------------------------------------------------------------------------
void FramePro::AddCustomStat(const wchar_t* p_name, int value, const wchar_t* p_graph, const wchar_t* p_unit, uint colour)
{
	AddCustomStat(p_name, (int64)value, p_graph, p_unit, colour);
}

//------------------------------------------------------------------------
void FramePro::AddCustomStat(const wchar_t* p_name, int64 value, const wchar_t* p_graph, const wchar_t* p_unit, uint colour)
{
	FrameProTLS* p_framepro_tls = GetFrameProTLS();
	if (!p_framepro_tls)
		return;

	// if we are connecting to FramePro, FramePro will ask for the string value later, otherwise we need to send it now
	if (p_framepro_tls->SendStringsImmediately())
		p_framepro_tls->SendString(p_name, PacketType::WStringPacket);

	p_framepro_tls->SetCustomStatInfo(p_name, p_graph, p_unit, colour);	// only sends first time

	CriticalSectionScope lock(p_framepro_tls->GetCurrentSendBufferCriticalSection());

	CustomStatPacketInt64* p_packet = p_framepro_tls->AllocateSpaceInBuffer<CustomStatPacketInt64>();

	CustomStatValueType::Enum value_type = CustomStatValueType::Int64;

	p_packet->m_PacketTypeAndValueType = PacketType::CustomStatPacketW | (value_type << 16);
	p_packet->m_Count = 1;
	p_packet->m_Name = (StringId)p_name;
	p_packet->m_Value = value;
	FRAMEPRO_GET_CLOCK_COUNT(p_packet->m_Time);
}

//------------------------------------------------------------------------
void FramePro::AddCustomStat(const wchar_t* p_name, float value, const wchar_t* p_graph, const wchar_t* p_unit, uint colour)
{
	AddCustomStat(p_name, (double)value, p_graph, p_unit, colour);
}

//------------------------------------------------------------------------
void FramePro::AddCustomStat(const wchar_t* p_name, double value, const wchar_t* p_graph, const wchar_t* p_unit, uint colour)
{
	FrameProTLS* p_framepro_tls = GetFrameProTLS();
	if (!p_framepro_tls)
		return;

	// if we are connecting to FramePro, FramePro will ask for the string value later, otherwise we need to send it now
	if (p_framepro_tls->SendStringsImmediately())
		p_framepro_tls->SendString(p_name, PacketType::StringPacket);

	p_framepro_tls->SetCustomStatInfo(p_name, p_graph, p_unit, colour);	// only sends first time

	CriticalSectionScope lock(p_framepro_tls->GetCurrentSendBufferCriticalSection());

	CustomStatPacketDouble* p_packet = p_framepro_tls->AllocateSpaceInBuffer<CustomStatPacketDouble>();

	CustomStatValueType::Enum value_type = CustomStatValueType::Double;

	p_packet->m_PacketTypeAndValueType = PacketType::CustomStatPacketW | (value_type << 16);
	p_packet->m_Count = 1;
	p_packet->m_Name = (StringId)p_name;
	p_packet->m_Value = value;
	FRAMEPRO_GET_CLOCK_COUNT(p_packet->m_Time);
}

//------------------------------------------------------------------------
void FramePro::AddCustomStat(StringId name, int value, const char* p_graph, const char* p_unit, uint colour)
{
	AddCustomStat(name, (int64)value, p_graph, p_unit, colour);
}

//------------------------------------------------------------------------
void FramePro::AddCustomStat(StringId name, int64 value, const char* p_graph, const char* p_unit, uint colour)
{
	FrameProTLS* p_framepro_tls = GetFrameProTLS();
	if (!p_framepro_tls)
		return;

	p_framepro_tls->SetCustomStatInfo(name, p_graph, p_unit, colour);	// only sends first time

	CriticalSectionScope lock(p_framepro_tls->GetCurrentSendBufferCriticalSection());

	CustomStatPacketInt64* p_packet = p_framepro_tls->AllocateSpaceInBuffer<CustomStatPacketInt64>();

	CustomStatValueType::Enum value_type = CustomStatValueType::Int64;

	p_packet->m_PacketTypeAndValueType = PacketType::CustomStatPacket | (value_type << 16);
	p_packet->m_Count = 1;
	p_packet->m_Name = name;
	p_packet->m_Value = value;
	FRAMEPRO_GET_CLOCK_COUNT(p_packet->m_Time);
}

//------------------------------------------------------------------------
void FramePro::AddCustomStat(StringId name, float value, const char* p_graph, const char* p_unit, uint colour)
{
	AddCustomStat(name, (double)value, p_graph, p_unit, colour);
}

//------------------------------------------------------------------------
void FramePro::AddCustomStat(StringId name, double value, const char* p_graph, const char* p_unit, uint colour)
{
	FrameProTLS* p_framepro_tls = GetFrameProTLS();
	if (!p_framepro_tls)
		return;

	p_framepro_tls->SetCustomStatInfo(name, p_graph, p_unit, colour);	// only sends first time

	CriticalSectionScope lock(p_framepro_tls->GetCurrentSendBufferCriticalSection());

	CustomStatPacketDouble* p_packet = p_framepro_tls->AllocateSpaceInBuffer<CustomStatPacketDouble>();

	CustomStatValueType::Enum value_type = CustomStatValueType::Double;

	p_packet->m_PacketTypeAndValueType = PacketType::CustomStatPacket | (value_type << 16);
	p_packet->m_Count = 1;
	p_packet->m_Name = name;
	p_packet->m_Value = value;
	FRAMEPRO_GET_CLOCK_COUNT(p_packet->m_Time);
}

//------------------------------------------------------------------------
void FramePro::AddCustomStat(StringId name, int value, const wchar_t* p_graph, const wchar_t* p_unit, uint colour)
{
	AddCustomStat(name, (int64)value, p_graph, p_unit, colour);
}

//------------------------------------------------------------------------
void FramePro::AddCustomStat(StringId name, int64 value, const wchar_t* p_graph, const wchar_t* p_unit, uint colour)
{
	FrameProTLS* p_framepro_tls = GetFrameProTLS();
	if (!p_framepro_tls)
		return;

	p_framepro_tls->SetCustomStatInfo(name, p_graph, p_unit, colour);	// only sends first time

	CriticalSectionScope lock(p_framepro_tls->GetCurrentSendBufferCriticalSection());

	CustomStatPacketInt64* p_packet = p_framepro_tls->AllocateSpaceInBuffer<CustomStatPacketInt64>();

	CustomStatValueType::Enum value_type = CustomStatValueType::Int64;

	p_packet->m_PacketTypeAndValueType = PacketType::CustomStatPacket | (value_type << 16);
	p_packet->m_Count = 1;
	p_packet->m_Name = name;
	p_packet->m_Value = value;
	FRAMEPRO_GET_CLOCK_COUNT(p_packet->m_Time);
}

//------------------------------------------------------------------------
void FramePro::AddCustomStat(StringId name, float value, const wchar_t* p_graph, const wchar_t* p_unit, uint colour)
{
	AddCustomStat(name, (double)value, p_graph, p_unit, colour);
}

//------------------------------------------------------------------------
void FramePro::AddCustomStat(StringId name, double value, const wchar_t* p_graph, const wchar_t* p_unit, uint colour)
{
	FrameProTLS* p_framepro_tls = GetFrameProTLS();
	if (!p_framepro_tls)
		return;

	p_framepro_tls->SetCustomStatInfo(name, p_graph, p_unit, colour);	// only sends first time

	CriticalSectionScope lock(p_framepro_tls->GetCurrentSendBufferCriticalSection());

	CustomStatPacketDouble* p_packet = p_framepro_tls->AllocateSpaceInBuffer<CustomStatPacketDouble>();

	CustomStatValueType::Enum value_type = CustomStatValueType::Double;

	p_packet->m_PacketTypeAndValueType = PacketType::CustomStatPacket | (value_type << 16);
	p_packet->m_Count = 1;
	p_packet->m_Name = name;
	p_packet->m_Value = value;
	FRAMEPRO_GET_CLOCK_COUNT(p_packet->m_Time);
}

//------------------------------------------------------------------------
void FramePro::AddCustomStat(StringId name, int value, StringId graph, StringId unit, uint colour)
{
	AddCustomStat(name, (int64)value, graph, unit, colour);
}

//------------------------------------------------------------------------
void FramePro::AddCustomStat(StringId name, int64 value, StringId graph, StringId unit, uint colour)
{
	FrameProTLS* p_framepro_tls = GetFrameProTLS();
	if (!p_framepro_tls)
		return;

	p_framepro_tls->SetCustomStatInfo(name, graph, unit, colour);	// only sends first time

	CriticalSectionScope lock(p_framepro_tls->GetCurrentSendBufferCriticalSection());

	CustomStatPacketInt64* p_packet = p_framepro_tls->AllocateSpaceInBuffer<CustomStatPacketInt64>();

	CustomStatValueType::Enum value_type = CustomStatValueType::Int64;

	p_packet->m_PacketTypeAndValueType = PacketType::CustomStatPacket | (value_type << 16);
	p_packet->m_Count = 1;
	p_packet->m_Name = name;
	p_packet->m_Value = value;
	FRAMEPRO_GET_CLOCK_COUNT(p_packet->m_Time);
}

//------------------------------------------------------------------------
void FramePro::AddCustomStat(StringId name, float value, StringId graph, StringId unit, uint colour)
{
	AddCustomStat(name, (double)value, graph, unit, colour);
}

//------------------------------------------------------------------------
void FramePro::AddCustomStat(StringId name, double value, StringId graph, StringId unit, uint colour)
{
	FrameProTLS* p_framepro_tls = GetFrameProTLS();
	if (!p_framepro_tls)
		return;

	p_framepro_tls->SetCustomStatInfo(name, graph, unit, colour);	// only sends first time

	CriticalSectionScope lock(p_framepro_tls->GetCurrentSendBufferCriticalSection());

	CustomStatPacketDouble* p_packet = p_framepro_tls->AllocateSpaceInBuffer<CustomStatPacketDouble>();

	CustomStatValueType::Enum value_type = CustomStatValueType::Double;

	p_packet->m_PacketTypeAndValueType = PacketType::CustomStatPacket | (value_type << 16);
	p_packet->m_Count = 1;
	p_packet->m_Name = name;
	p_packet->m_Value = value;
	FRAMEPRO_GET_CLOCK_COUNT(p_packet->m_Time);
}

//------------------------------------------------------------------------
void FramePro::SetThreadName(const char* p_name)
{
	GetFrameProSession().SetThreadName(p_name);
}

//------------------------------------------------------------------------
void FramePro::SetThreadName(int thread_id, const char* p_name)
{
	GetFrameProSession().SetThreadName(thread_id, p_name);
}

//------------------------------------------------------------------------
void FramePro::SetThreadOrder(StringId thread_name)
{
	FrameProTLS* p_framepro_tls = GetFrameProTLS();
	if (p_framepro_tls)
		p_framepro_tls->SetThreadOrder(thread_name);
}

//------------------------------------------------------------------------
FramePro::StringId FramePro::RegisterString(const char* p_str)
{
	FrameProTLS* p_framepro_tls = GetFrameProTLS();
	return p_framepro_tls ? p_framepro_tls->RegisterString(p_str) : 0;
}

//------------------------------------------------------------------------
FramePro::StringId FramePro::RegisterString(const wchar_t* p_str)
{
	FrameProTLS* p_framepro_tls = GetFrameProTLS();
	return p_framepro_tls ? p_framepro_tls->RegisterString(p_str) : 0;
}

//------------------------------------------------------------------------
void FramePro::StartRecording(const char* p_filename, bool context_switches, bool callstacks, int64 max_file_size)
{
	GetFrameProSession().StartRecording(p_filename, context_switches, callstacks, max_file_size);
}

//------------------------------------------------------------------------
void FramePro::StartRecording(const wchar_t* p_filename, bool context_switches, bool callstacks, int64 max_file_size)
{
	GetFrameProSession().StartRecording(p_filename, context_switches, callstacks, max_file_size);
}

//------------------------------------------------------------------------
void FramePro::StopRecording()
{
	GetFrameProSession().StopRecording();
}

//------------------------------------------------------------------------
void FramePro::SetThreadPriority(int priority)
{
	GetFrameProSession().SetThreadPriority(priority);
}

//------------------------------------------------------------------------
void FramePro::SetThreadAffinity(int affinity)
{
	GetFrameProSession().SetThreadAffinity(affinity);
}

//------------------------------------------------------------------------
void FramePro::BlockSockets()
{
	GetFrameProSession().BlockSockets();
}

//------------------------------------------------------------------------
void FramePro::UnblockSockets()
{
	GetFrameProSession().UnblockSockets();
}

//------------------------------------------------------------------------
void FramePro::SetPort(int port)
{
	GetFrameProSession().SetPort(port);
}

//------------------------------------------------------------------------
void FramePro::SendSessionInfo(const char* p_name, const char* p_value)
{
	GetFrameProSession().SendSessionInfo(p_name, p_value);
}

//------------------------------------------------------------------------
void FramePro::SendSessionInfo(const wchar_t* p_name, const wchar_t* p_value)
{
	GetFrameProSession().SendSessionInfo(p_name, p_value);
}

//------------------------------------------------------------------------
void FramePro::AddCustomStatTimer(CustomStatTimer* p_timer)
{
	GetFrameProSession().AddCustomStatTimer(p_timer);
}

//------------------------------------------------------------------------
void FramePro::CleanupThread()
{
	FrameProTLS* p_framepro_tls = TryGetFrameProTLS();

	if (p_framepro_tls)
	{
		p_framepro_tls->FlushCurrentSendBuffer();

		p_framepro_tls->Shutdown();		// will get cleaned up the next time the buffers are sent on the send thread

		ClearFrameProTLS();
	}
}

//------------------------------------------------------------------------
void FramePro::PushConditionalParentScope(const char* p_name, int64 pre_duration, int64 post_duration)
{
	FrameProTLS* p_framepro_tls = GetFrameProTLS();
	if (p_framepro_tls)
		p_framepro_tls->PushConditionalParentScope(p_name, pre_duration, post_duration);
}

//------------------------------------------------------------------------
void FramePro::PopConditionalParentScope(bool add_children)
{
	FrameProTLS* p_framepro_tls = GetFrameProTLS();
	if (p_framepro_tls)
		p_framepro_tls->PopConditionalParentScope(add_children);
}

//------------------------------------------------------------------------
bool FramePro::CallConditionalParentScopeCallback(ConditionalParentScopeCallback p_callback, const char* p_name, int64 start_time, int64 end_time)
{
	return GetFrameProSession().CallConditionalParentScopeCallback(p_callback, p_name, start_time, end_time);
}

//------------------------------------------------------------------------
void FramePro::StartHiResTimer(const char* p_name)
{
	FrameProTLS* p_framepro_tls = GetFrameProTLS();
	if (p_framepro_tls)
		p_framepro_tls->StartHiResTimer(p_name);
}

//------------------------------------------------------------------------
void FramePro::StopHiResTimer()
{
	FrameProTLS* p_framepro_tls = GetFrameProTLS();
	if (p_framepro_tls)
		p_framepro_tls->StopHiResTimer();
}

//------------------------------------------------------------------------
void FramePro::Log(const char* p_message)
{
	if (g_Connected)
	{
		FrameProTLS* p_framepro_tls = GetFrameProTLS();
		if (p_framepro_tls)
			p_framepro_tls->SendLogPacket(p_message);
	}
}

//------------------------------------------------------------------------
void FramePro::AddEvent(const char* p_name, uint colour)
{
	if (g_Connected)
	{
		FrameProTLS* p_framepro_tls = GetFrameProTLS();
		if (p_framepro_tls)
			p_framepro_tls->SendEventPacket(p_name, colour);
	}
}

//------------------------------------------------------------------------
void FramePro::AddWaitEvent(
	int64 event_id,
	int64 start_time,
	int start_core,
	int64 end_time,
	int end_core)
{
	FrameProTLS* p_framepro_tls = GetFrameProTLS();
	if (!p_framepro_tls)
		return;

#ifdef FRAMEPRO_SCOPE_MIN_TIME
	if (end_time - start_time < p_framepro_tls->GetWaitEventMinTime())
		return;
#endif
	SendWaitEventPacket(event_id, start_time, start_core, PacketType::StartWaitEventPacket);
	SendWaitEventPacket(event_id, end_time, end_core, PacketType::StopWaitEventPacket);
}

//------------------------------------------------------------------------
void FramePro::TriggerWaitEvent(int64 event_id)
{
	if (!g_Connected)
		return;

	int64 time;
	FRAMEPRO_GET_CLOCK_COUNT(time);

	SendWaitEventPacket(event_id, time, Platform::GetCore(), PacketType::TriggerWaitEventPacket);
}

//------------------------------------------------------------------------
void FramePro::SetScopeCustomStat(const char* p_name, int64 value, const char* p_graph, const char* p_unit, uint colour)
{
	FrameProTLS* p_framepro_tls = GetFrameProTLS();
	if (!p_framepro_tls)
		return;

	if (p_framepro_tls->SendStringsImmediately())
		p_framepro_tls->SendString(p_name, PacketType::StringPacket);

	p_framepro_tls->SetCustomTimeSpanStat((StringId)p_name, value);

	AddCustomStat(p_name, value, p_graph, p_unit, colour);
}

//------------------------------------------------------------------------
void FramePro::SetScopeCustomStat(const wchar_t* p_name, int64 value, const wchar_t* p_graph, const wchar_t* p_unit, uint colour)
{
	FrameProTLS* p_framepro_tls = GetFrameProTLS();
	if (!p_framepro_tls)
		return;

	if (p_framepro_tls->SendStringsImmediately())
		p_framepro_tls->SendString(p_name, PacketType::WStringPacket);

	p_framepro_tls->SetCustomTimeSpanStatW((StringId)p_name, value);

	AddCustomStat(p_name, value, p_graph, p_unit, colour);
}

//------------------------------------------------------------------------
void FramePro::SetScopeCustomStat(StringId name, int64 value, StringId graph, StringId unit, uint colour)
{
	FrameProTLS* p_framepro_tls = GetFrameProTLS();
	if (!p_framepro_tls)
		return;

	// don't care about whether it is W or not because string has already been registerd
	p_framepro_tls->SetCustomTimeSpanStat(name, value);

	AddCustomStat(name, value, graph, unit, colour);
}

//------------------------------------------------------------------------
void FramePro::SetScopeCustomStat(const char* p_name, int value, const char* p_graph, const char* p_unit, uint colour)
{
	SetScopeCustomStat(p_name, (int64)value, p_graph, p_unit, colour);
}

//------------------------------------------------------------------------
void FramePro::SetScopeCustomStat(const wchar_t* p_name, int value, const wchar_t* p_graph, const wchar_t* p_unit, uint colour)
{
	SetScopeCustomStat(p_name, (int64)value, p_graph, p_unit, colour);
}

//------------------------------------------------------------------------
void FramePro::SetScopeCustomStat(StringId name, int value, StringId graph, StringId unit, uint colour)
{
	SetScopeCustomStat(name, (int64)value, graph, unit, colour);
}

//------------------------------------------------------------------------
void FramePro::SetScopeCustomStat(const char* p_name, float value, const char* p_graph, const char* p_unit, uint colour)
{
	SetScopeCustomStat(p_name, (double)value, p_graph, p_unit, colour);
}

//------------------------------------------------------------------------
void FramePro::SetScopeCustomStat(const wchar_t* p_name, float value, const wchar_t* p_graph, const wchar_t* p_unit, uint colour)
{
	SetScopeCustomStat(p_name, (double)value, p_graph, p_unit, colour);
}

//------------------------------------------------------------------------
void FramePro::SetScopeCustomStat(StringId name, float value, StringId graph, StringId unit, uint colour)
{
	SetScopeCustomStat(name, (double)value, graph, unit, colour);
}

//------------------------------------------------------------------------
void FramePro::SetScopeCustomStat(const char* p_name, double value, const char* p_graph, const char* p_unit, uint colour)
{
	FrameProTLS* p_framepro_tls = GetFrameProTLS();
	if (!p_framepro_tls)
		return;

	if (p_framepro_tls->SendStringsImmediately())
		p_framepro_tls->SendString(p_name, PacketType::StringPacket);

	p_framepro_tls->SetCustomTimeSpanStat((StringId)p_name, value);

	AddCustomStat(p_name, value, p_graph, p_unit, colour);
}

//------------------------------------------------------------------------
void FramePro::SetScopeCustomStat(const wchar_t* p_name, double value, const wchar_t* p_graph, const wchar_t* p_unit, uint colour)
{
	FrameProTLS* p_framepro_tls = GetFrameProTLS();
	if (!p_framepro_tls)
		return;

	if (p_framepro_tls->SendStringsImmediately())
		p_framepro_tls->SendString(p_name, PacketType::StringPacket);

	p_framepro_tls->SetCustomTimeSpanStat((StringId)p_name, value);

	AddCustomStat(p_name, value, p_graph, p_unit, colour);
}

//------------------------------------------------------------------------
void FramePro::SetScopeCustomStat(StringId name, double value, StringId graph, StringId unit, uint colour)
{
	FrameProTLS* p_framepro_tls = GetFrameProTLS();
	if (!p_framepro_tls)
		return;

	p_framepro_tls->SetCustomTimeSpanStat(name, value);

	AddCustomStat(name, value, graph, unit, colour);
}

//------------------------------------------------------------------------
void FramePro::SetConditionalScopeMinTimeInMicroseconds(int64 value)
{
	GetFrameProSession().SetConditionalScopeMinTimeInMicroseconds(value);
}

//------------------------------------------------------------------------
void FramePro::SetScopeColour(StringId name, uint colour)
{
	GetFrameProSession().SetScopeColour(name, colour);
}

//------------------------------------------------------------------------
void FramePro::SetCustomStatGraph(StringId name, StringId graph)
{
	GetFrameProSession().SetCustomStatGraph(name, graph);
}

//------------------------------------------------------------------------
void FramePro::SetCustomStatUnit(StringId name, StringId unit)
{
	GetFrameProSession().SetCustomStatUnit(name, unit);
}

//------------------------------------------------------------------------
void FramePro::SetCustomStatColour(StringId name, uint colour)
{
	GetFrameProSession().SetCustomStatColour(name, colour);
}

//------------------------------------------------------------------------
void FramePro::SendScopeCallstack()
{
	GetFrameProSession().SendScopeCallstack();
}

//------------------------------------------------------------------------
int FramePro::GetCore()
{
	return Platform::GetCore();
}

//------------------------------------------------------------------------
#if FRAMEPRO_DEBUG || FRAMEPRO_ALLOW_UNPARENTED_HIRES_SCOPES
void FramePro::StartScope()
{
	FrameProTLS* p_framepro_tls = GetFrameProTLS();
	if (p_framepro_tls)
		p_framepro_tls->IncScopeDepth();
}
#endif

//------------------------------------------------------------------------
#if FRAMEPRO_DEBUG || FRAMEPRO_ALLOW_UNPARENTED_HIRES_SCOPES
void FramePro::StopScope()
{
	FrameProTLS* p_framepro_tls = GetFrameProTLS();
	if (p_framepro_tls)
		p_framepro_tls->DecScopeDepth();
}
#endif


//------------------------------------------------------------------------
//
// FrameProCallstackSet.cpp
//

//------------------------------------------------------------------------
namespace FramePro
{
	//------------------------------------------------------------------------
	const int g_CallstackSetInitialCapacity = 4096;		// must be a power of 2

	//------------------------------------------------------------------------
	std::atomic<int> CallstackSet::m_NextUniqueId(0);

	//------------------------------------------------------------------------
	inline bool StacksMatch(FramePro::Callstack* p_callstack, uint64* p_stack, int stack_size, unsigned int hash)
	{
		if(p_callstack->m_Size != stack_size)
			return false;

		if(p_callstack->m_Hash != hash)
			return false;

		for(int i=0; i<stack_size; ++i)
			if(p_callstack->mp_Stack[i] != p_stack[i])
				return false;

		return true;
	}

	//------------------------------------------------------------------------
	CallstackSet::CallstackSet(Allocator* p_allocator)
	:	mp_Data((Callstack**)p_allocator->Alloc(g_CallstackSetInitialCapacity*sizeof(Callstack*))),
		m_CapacityMask(g_CallstackSetInitialCapacity-1),
		m_Count(0),
		m_Capacity(g_CallstackSetInitialCapacity),
		mp_Allocator(p_allocator),
		m_BlockAllocator(p_allocator)
	{
		if(mp_Data)
			memset(mp_Data, 0, g_CallstackSetInitialCapacity*sizeof(Callstack*));
	}

	//------------------------------------------------------------------------
	CallstackSet::~CallstackSet()
	{
		Clear();
	}

	//------------------------------------------------------------------------
	void CallstackSet::Grow()
	{
		int old_capacity = m_Capacity;
		Callstack** p_old_data = mp_Data;

		// allocate a new set
		m_Capacity *= 2;
		m_CapacityMask = m_Capacity - 1;
		int size = m_Capacity * sizeof(Callstack*);

		mp_Data = (Callstack**)mp_Allocator->Alloc(size);
		if (!mp_Data)
			return;

		memset(mp_Data, 0, size);

		// transfer callstacks from old set
		m_Count = 0;
		for(int i=0; i<old_capacity; ++i)
		{
			Callstack* p_callstack = p_old_data[i];
			if(p_callstack)
				Add(p_callstack);
		}

		// release old buffer
		mp_Allocator->Free(p_old_data);
	}

	//------------------------------------------------------------------------
	FramePro::Callstack* CallstackSet::Get(uint64* p_stack, int stack_size, unsigned int hash)
	{
		if (!mp_Data)
			return NULL;

		int index = hash & m_CapacityMask;

		while(mp_Data[index] && !StacksMatch(mp_Data[index], p_stack, stack_size, hash))
			index = (index + 1) & m_CapacityMask;

		return mp_Data[index];
	}

	//------------------------------------------------------------------------
	FramePro::Callstack* CallstackSet::Add(uint64* p_stack, int stack_size, unsigned int hash)
	{
		// grow the set if necessary
		if(m_Count > m_Capacity/4)
			Grow();

		// create a new callstack
		Callstack* p_callstack = (Callstack*)m_BlockAllocator.Alloc(sizeof(Callstack));
		p_callstack->m_ID = m_NextUniqueId++;
		p_callstack->m_Size = stack_size;
		p_callstack->mp_Stack = (uint64*)m_BlockAllocator.Alloc(stack_size*sizeof(uint64));
		p_callstack->m_Hash = hash;
		memcpy(p_callstack->mp_Stack, p_stack, stack_size*sizeof(uint64));

		Add(p_callstack);

		return p_callstack;
	}

	//------------------------------------------------------------------------
	void CallstackSet::Add(Callstack* p_callstack)
	{
		if (!mp_Data)
			return;

		// find a clear index
		int index = p_callstack->m_Hash & m_CapacityMask;
		while(mp_Data[index])
			index = (index + 1) & m_CapacityMask;

		mp_Data[index] = p_callstack;

		++m_Count;
	}

	//------------------------------------------------------------------------
	void CallstackSet::Clear()
	{
		m_BlockAllocator.Clear();

		mp_Allocator->Free(mp_Data);

		size_t size = g_CallstackSetInitialCapacity*sizeof(Callstack*);
		
		mp_Data = (Callstack**)mp_Allocator->Alloc((int)size);
		
		if (mp_Data)
		{
			memset(mp_Data, 0, size);
			m_CapacityMask = g_CallstackSetInitialCapacity - 1;
			m_Capacity = g_CallstackSetInitialCapacity;
		}
		else
		{
			m_Capacity = 0;
		}

		m_Count = 0;
	}
}


//------------------------------------------------------------------------
//
// FrameProLib.cpp
//

//------------------------------------------------------------------------
#include <stdarg.h>

//------------------------------------------------------------------------
namespace FramePro
{
	//------------------------------------------------------------------------
	void SPrintf(char* p_buffer, size_t const buffer_size, const char* p_format, ...)
	{
		va_list args;
		va_start(args, p_format);

		Platform::VSPrintf(p_buffer, buffer_size, p_format, args);

		va_end(args);
	}

	//------------------------------------------------------------------------
	void DebugWrite(const char* p_str, ...)
	{
		va_list args;
		va_start(args, p_str);

		static char temp_string[1024];
		Platform::VSPrintf(temp_string, sizeof(temp_string), p_str, args);

		Platform::DebugWrite(temp_string);

		va_end(args);
	}
}


//------------------------------------------------------------------------
//
// FrameProSession.cpp
//

//------------------------------------------------------------------------
namespace FramePro
{
	//------------------------------------------------------------------------
	const char* g_NonInteractiveRecordingFilePath = "framepro_recording.bin";

	//------------------------------------------------------------------------
	FrameProSession* FrameProSession::mp_Inst = NULL;

	//------------------------------------------------------------------------
	class DefaultAllocator : public Allocator
	{
	public:
		void* Alloc(size_t size) { return new char[size]; }
		void Free(void* p) { delete[] (char*)p; }
	};

	//------------------------------------------------------------------------
	void GetDateString(char* p_date, size_t len)
	{
		time_t rawtime;
		time(&rawtime);

		tm timeinfo;
		Platform::GetLocalTime(&timeinfo, &rawtime);

		strftime(p_date, len, "%d-%m-%Y %I:%M:%S", &timeinfo);
	}

	//------------------------------------------------------------------------
	FrameProSession::FrameProSession()
	:	mp_CreatedAllocator(NULL),
		m_Initialised(false),
		m_InitialiseConnectionNextFrame(false),
		m_StartContextSwitchRecording(false),
#if FRAMEPRO_ENABLE_CALLSTACKS
		m_StartRecordingCallstacks(false),
#endif
		m_ClockFrequency(0),
		m_MainThreadId(-1),
		m_SendThreadStarted(false, true),
		m_SendReady(false, true),
		m_SendComplete(false, false),
		m_ReceiveThreadTerminatedEvent(false, false),
		m_Interactive(true),
		m_NonInteractiveRecordingFileSize(0),
		m_LastSessionInfoSendTime(0),
		m_RecordingFileSize(0),
		m_MaxRecordingFileSize(0),
		m_ThreadPrioritySet(false),
		m_ThreadPriority(0),
		m_ThreadAffinitySet(false),
		m_ThreadAffinity(0),
		m_SendThreadExit(false),
		m_SendThreadFinished(false, true),
		m_SocketsBlocked(FRAMEPRO_SOCKETS_BLOCKED_BY_DEFAULT),
		m_MainThreadSendBufferIndex(0),
		m_CustomStatTimers(NULL),
		mp_ContextSwitchRecorder(NULL)
#if FRAMEPRO_ENABLE_CALLSTACKS
		,m_SendModules(false)
#endif
	{
		mp_Inst = this;

		FRAMEPRO_ASSERT(sizeof(m_Port) >= strlen(FRAMEPRO_PORT) + 1);
		memcpy(m_Port, FRAMEPRO_PORT, strlen(FRAMEPRO_PORT) + 1);

		CalculateTimerFrequency();
	}

	//------------------------------------------------------------------------
	FrameProSession::~FrameProSession()
	{
		Disconnect(true);

		// shut down the connect thread
		m_ListenSocket.Disconnect();
		m_ConnectThread.WaitForThreadToTerminate(10 * 1000);

		m_NamedThreads.Clear();

		Platform::DestroyContextSwitchRecorder(mp_ContextSwitchRecorder, GetAllocator());

		// must clear all arrays and buffers and detach the allocator before deleting the allocator
		m_ProcessIds.Clear();
		m_ProcessIds.SetAllocator(NULL);

		{
			CriticalSectionScope tls_lock(m_MainThreadSendBufferLock);
			for (Buffer& buffer : m_MainThreadSendBuffer)
			{
				buffer.ClearAndFree();
				buffer.SetAllocator(NULL);
			}
		}

		m_StringRequestPackets.Clear();
		m_StringRequestPackets.SetAllocator(NULL);

		m_ModulePackets.Clear();
		m_ModulePackets.SetAllocator(NULL);

		m_ScopeColours.Clear();
		m_ScopeColours.SetAllocator(NULL);

		m_CustomStatGraphs.Clear();
		m_CustomStatGraphs.SetAllocator(NULL);

		m_CustomStatUnits.Clear();
		m_CustomStatUnits.SetAllocator(NULL);

		m_CustomStatColours.Clear();
		m_CustomStatColours.SetAllocator(NULL);

		m_NamedThreads.Clear();
		m_NamedThreads.SetAllocator(NULL);

		m_Connectionchangedcallbacks.Clear();
		m_Connectionchangedcallbacks.SetAllocator(NULL);

		delete mp_CreatedAllocator;
	}

	//------------------------------------------------------------------------
	void FrameProSession::SetPort(int port)
	{
		Platform::ToString(port, m_Port, sizeof(m_Port));
	}

	//------------------------------------------------------------------------
	void FrameProSession::SetAllocator(Allocator* p_allocator)
	{
		if(m_Allocator.HasAllocator())
			FramePro::DebugBreak();		// allocator already set. You must call Allocator BEFORE calling FRAMEPRO_FRAME_START

		SetAllocatorInternal(p_allocator);
	}

	//------------------------------------------------------------------------
	void FrameProSession::SetAllocatorInternal(Allocator* p_allocator)
	{
		FRAMEPRO_ASSERT(!m_Allocator.HasAllocator());
		FRAMEPRO_ASSERT(p_allocator);

		m_Allocator.SetAllocator(p_allocator);

		m_NonInteractiveRecordingFile.SetAllocator(p_allocator);
		m_RecordingFile.SetAllocator(p_allocator);
	}

	//------------------------------------------------------------------------
	void FrameProSession::CreateDefaultAllocator()
	{
		CriticalSectionScope lock(m_CriticalSection);

		if(!m_Allocator.HasAllocator())
		{
			mp_CreatedAllocator = new DefaultAllocator();
			SetAllocatorInternal(mp_CreatedAllocator);
		}
	}

	//------------------------------------------------------------------------
	void FrameProSession::CalculateTimerFrequency()
	{
		m_ClockFrequency = Platform::GetTimerFrequency();
	}

	//------------------------------------------------------------------------
	int FrameProSession::StaticSendThreadMain(void* p_arg)
	{
		FrameProSession* p_this = (FrameProSession*)p_arg;
		int ret = p_this->SendThreadMain();

		#if !FRAMEPRO_PLATFORM_UE4
			DestroyFrameProTLS(GetFrameProTLS());
			ClearFrameProTLS();
		#endif

		return ret;
	}

	//------------------------------------------------------------------------
	int FrameProSession::SendThreadMain()
	{
		SetThreadName("FramePro Send Thread");

		m_SendThreadStarted.Set();

		m_SendReady.Wait();

		while(!m_SendThreadExit)
		{
			int64 start_time;
			FRAMEPRO_GET_CLOCK_COUNT(start_time);

			{
				FRAMEPRO_NAMED_SCOPE("FramePro Send");
				SendFrameBuffer();
			}

			int64 end_time;
			FRAMEPRO_GET_CLOCK_COUNT(end_time);

			m_SendComplete.Set();

			int sleep_time = FRAMEPRO_MAX_SEND_DELAY - (int)((end_time - start_time) * 1000 / m_ClockFrequency);
			if(sleep_time > 0)
				m_SendReady.Wait(sleep_time);
		}

		SendFrameBuffer();

		m_SendComplete.Set();

		m_SendThreadFinished.Set();

		return 0;
	}

	//------------------------------------------------------------------------
#if FRAMEPRO_SOCKETS_ENABLED
	int FrameProSession::StaticConnectThreadMain(void* p_arg)
	{
		FrameProSession* p_this = (FrameProSession*)p_arg;
		int ret = p_this->ConnectThreadMain();

		#if !FRAMEPRO_PLATFORM_UE4
			FrameProTLS* p_framepro_tls = TryGetFrameProTLS();
			if (p_framepro_tls)
			{
				DestroyFrameProTLS(p_framepro_tls);
				ClearFrameProTLS();
			}
		#endif

		return ret;
	}
#endif

	//------------------------------------------------------------------------
#if FRAMEPRO_SOCKETS_ENABLED
	int FrameProSession::ConnectThreadMain()
	{
		if(m_SocketsBlocked)
			return 0;

		{
			CriticalSectionScope lock(m_CriticalSection);
			if(m_RecordingFile.IsOpened())
			{
				m_ListenSocket.Disconnect();		// don't allow connections while recording
				return 0;
			}
		}

		bool accepted = m_ListenSocket.Accept(m_ClientSocket);

		if(accepted)
			m_InitialiseConnectionNextFrame = true;

		return 0;
	}
#endif

	//------------------------------------------------------------------------
#if FRAMEPRO_SOCKETS_ENABLED
	int FrameProSession::StaticReceiveThreadMain(void* p_arg)
	{
		FrameProSession* p_this = (FrameProSession*)p_arg;
		int ret = p_this->ReceiveThreadMain();

		#if !FRAMEPRO_PLATFORM_UE4
			FrameProTLS* p_framepro_tls = GetFrameProTLS();
			if (p_framepro_tls)
			{
				DestroyFrameProTLS(p_framepro_tls);
				ClearFrameProTLS();
			}
		#endif

		return ret;
	}
#endif

	//------------------------------------------------------------------------
	void FrameProSession::SendOnMainThread(void* p_src, int size)
	{
		CriticalSectionScope tls_lock(m_MainThreadSendBufferLock);

		void* p_dst = m_MainThreadSendBuffer[m_MainThreadSendBufferIndex].Allocate(size);
		if(p_dst)
			memcpy(p_dst, p_src, size);
	}

	//------------------------------------------------------------------------
#if FRAMEPRO_SOCKETS_ENABLED
	int FrameProSession::ReceiveThreadMain()
	{
		while(g_Connected)
		{
			int packet_type;

			if (m_ClientSocket.Receive(&packet_type, sizeof(packet_type)) != sizeof(packet_type))
			{
				m_ReceiveThreadTerminatedEvent.Set();
				return OnReceiveThreadExit();
			}

			int padding = 0;
			if (m_ClientSocket.Receive(&padding, sizeof(padding)) != sizeof(padding))
			{
				m_ReceiveThreadTerminatedEvent.Set();
				return OnReceiveThreadExit();
			}

			switch(packet_type)
			{
				case PacketType::RequestStringLiteralPacket:
				{
					RequestStringLiteralPacket packet;
					if (m_ClientSocket.Receive(&packet, sizeof(packet)) != sizeof(packet))
					{
						m_ReceiveThreadTerminatedEvent.Set();
						return OnReceiveThreadExit();
					}

					{
						CriticalSectionScope tls_lock(m_StringRequestPacketsLock);
						m_StringRequestPackets.Add(packet);
					}
				} break;

				case PacketType::SetConditionalScopeMinTimePacket:
				{
					SetConditionalScopeMinTimePacket packet;
					if (m_ClientSocket.Receive(&packet, sizeof(packet)) != sizeof(packet))
					{
						m_ReceiveThreadTerminatedEvent.Set();
						return OnReceiveThreadExit();
					}

					g_ConditionalScopeMinTime = packet.m_MinTime;
				} break;

				case PacketType::ConnectResponsePacket:
				{
					ConnectResponsePacket packet;
					if (m_ClientSocket.Receive(&packet, sizeof(packet)) != sizeof(packet))
					{
						m_ReceiveThreadTerminatedEvent.Set();
						return OnReceiveThreadExit();
					}

					{
						CriticalSectionScope lock(m_SendFrameBufferCriticalSection);

						if(!packet.m_Interactive)
						{
							bool opened = m_NonInteractiveRecordingFile.OpenForWrite(g_NonInteractiveRecordingFilePath);
							FRAMEPRO_ASSERT(opened);
							FRAMEPRO_UNREFERENCED(opened);
						}

						m_Interactive = packet.m_Interactive ? 1 : 0;

						{
							CriticalSectionScope tls_lock(m_TLSListCriticalSection);
							for(FrameProTLS* p_iter=m_FrameProTLSList.GetHead(); p_iter!=NULL; p_iter=p_iter->GetNext())
								p_iter->SetInteractive(m_Interactive);
						}
					}

					if(packet.m_RecordContextSwitches)
						StartRecordingContextSitches();

				} break;

				case PacketType::RequestRecordedDataPacket:
				{
					SendRecordedDataAndDisconnect();
				} break;

				case PacketType::SetCallstackRecordingEnabledPacket:
				{
					SetCallstackRecordingEnabledPacket packet;
					if (m_ClientSocket.Receive(&packet, sizeof(packet)) == sizeof(packet))
					{
						#if FRAMEPRO_ENABLE_CALLSTACKS
							SetCallstacksEnabled(packet.m_Enabled != 0);
						#endif
					}
				} break;
			}
		}

		m_ReceiveThreadTerminatedEvent.Set();
		return 0;
	}
#endif

	//------------------------------------------------------------------------
#if FRAMEPRO_ENABLE_CALLSTACKS
	void FrameProSession::SetCallstacksEnabled(bool enabled)
	{
		if (enabled)
			SendModules();

		{
			CriticalSectionScope tls_lock(m_TLSListCriticalSection);
			for (FrameProTLS* p_tls = m_FrameProTLSList.GetHead(); p_tls != NULL; p_tls = p_tls->GetNext())
				p_tls->SetSendCallstacks(enabled);
		}
	}
#endif

	//------------------------------------------------------------------------
#if FRAMEPRO_ENABLE_CALLSTACKS
	void FrameProSession::SendModules()
	{
		// enumerate modules
		if (!m_SendModules)
		{
			Platform::EnumerateModules(m_ModulePackets, GetAllocator());

			// send module packets
			for (int i = 0; i < m_ModulePackets.GetCount(); ++i)
			{
				ModulePacket* p_module_packet = m_ModulePackets[i];
				SendImmediate(p_module_packet, sizeof(ModulePacket), GetFrameProTLS());
				m_Allocator.Free(p_module_packet);
			}
			m_ModulePackets.Clear();

			m_SendModules = true;
		}
	}
#endif

	//------------------------------------------------------------------------
	void FrameProSession::StartRecordingContextSitches()
	{
		DynamicString error(GetAllocator());

		if (!mp_ContextSwitchRecorder)
			mp_ContextSwitchRecorder = Platform::CreateContextSwitchRecorder(GetAllocator());

		if (!mp_ContextSwitchRecorder)
			return;

		bool started = Platform::StartRecordingContextSitches(mp_ContextSwitchRecorder, ContextSwitchCallback_Static, this, error, GetAllocator());

		if (!started)
		{
			Platform::DestroyContextSwitchRecorder(mp_ContextSwitchRecorder, GetAllocator());
			mp_ContextSwitchRecorder = NULL;
		}

		// send the context switch started packet
		ContextSwitchRecordingStartedPacket response_packet;
		response_packet.m_PacketType = PacketType::ContextSwitchRecordingStartedPacket;
		response_packet.m_StartedSucessfully = started;
		
		error.CopyTo(response_packet.m_Error, sizeof(response_packet.m_Error));

		SendOnMainThread(response_packet);
	}

	//------------------------------------------------------------------------
#if FRAMEPRO_SOCKETS_ENABLED
	int FrameProSession::OnReceiveThreadExit()
	{
		Disconnect();
		return 0;
	}
#endif

	//------------------------------------------------------------------------
	void FrameProSession::CreateSendThread()
	{
		m_CriticalSection.Leave();

		if (!m_SendThread.CreateThread(StaticSendThreadMain, this, GetAllocator()))
			return;

		if(m_ThreadPrioritySet)
			m_SendThread.SetPriority(m_ThreadPriority);

		if(m_ThreadAffinitySet)
			m_SendThread.SetAffinity(m_ThreadAffinity);
	
		m_SendThreadStarted.Wait();

		m_CriticalSection.Enter();
	}

	//------------------------------------------------------------------------
#if FRAMEPRO_SOCKETS_ENABLED
	void FrameProSession::CreateReceiveThread()
	{
		m_ReceiveThreadTerminatedEvent.Reset();

		if (!m_ReceiveThread.CreateThread(StaticReceiveThreadMain, this, GetAllocator()))
			return;

		if(m_ThreadPrioritySet)
			m_ReceiveThread.SetPriority(m_ThreadPriority);

		if(m_ThreadAffinitySet)
			m_ReceiveThread.SetAffinity(m_ThreadAffinity);
	}
#endif

	//------------------------------------------------------------------------
	void FrameProSession::ContextSwitchCallback_Static(const ContextSwitch& context_switch, void* p_param)
	{
		FrameProSession* p_this = (FrameProSession*)p_param;
		p_this->ContextSwitchCallback(context_switch);
	}

	//------------------------------------------------------------------------
	void FrameProSession::ContextSwitchCallback(const ContextSwitch& context_switch)
	{
		FrameProTLS* p_framepro_tls = GetFrameProTLS();

		// send the process name string
		if(!m_ProcessIds.Contains(context_switch.m_ProcessId))
		{
			m_ProcessIds.SetAllocator(GetAllocator());

			if (!m_ProcessIds.Add(context_switch.m_ProcessId))
				return;

			const int max_process_name_length = 260;
			char process_name[max_process_name_length];
			if(Platform::GetProcessName(context_switch.m_ProcessId, process_name, max_process_name_length))
			{
				StringId name_id = RegisterString(process_name);
				p_framepro_tls->SendSessionInfoPacket(ProcessNamePacket(context_switch.m_ProcessId, name_id));
			}
		}

		CriticalSectionScope lock(p_framepro_tls->GetCurrentSendBufferCriticalSection());

		// send the context switch packet
		ContextSwitchPacket* p_packet = p_framepro_tls->AllocateSpaceInBuffer<ContextSwitchPacket>();

		p_packet->m_PacketType = PacketType::ContextSwitchPacket;
		p_packet->m_ProcessId = context_switch.m_ProcessId;
		p_packet->m_CPUId = context_switch.m_CPUId;
		p_packet->m_Timestamp = context_switch.m_Timestamp;
		p_packet->m_OldThreadId = context_switch.m_OldThreadId;
		p_packet->m_NewThreadId = context_switch.m_NewThreadId;
		p_packet->m_OldThreadState = context_switch.m_OldThreadState;
		p_packet->m_OldThreadWaitReason = context_switch.m_OldThreadWaitReason;
		p_packet->m_Padding = 0;
	}

	//------------------------------------------------------------------------
	bool FrameProSession::InitialiseConnection(FrameProTLS* p_framepro_tls)
	{
		// start the Send thread FIRST, but paused (because it adds another TLS that we need to call OnConnected on)
		m_SendComplete.Reset();
		m_SendReady.Reset();
		CreateSendThread();

		// call OnConnected on all TLS threads
		bool recording_to_file = m_RecordingFile.IsOpened();
		{
			CriticalSectionScope tls_lock(m_TLSListCriticalSection);
			for (FrameProTLS* p_tls = m_FrameProTLSList.GetHead(); p_tls != NULL; p_tls = p_tls->GetNext())
			{
				if (!p_tls->OnConnected(recording_to_file))
					return false;
			}
		}

		p_framepro_tls->SendConnectPacket(m_ClockFrequency, Platform::GetCurrentProcessId(), Platform::GetPlatformEnum());

#if FRAMEPRO_ENABLE_CALLSTACKS
		m_SendModules = false;
#endif
		// tell the send thread that there is data ready and wait for it to be sent
		m_SendReady.Set();
		m_CriticalSection.Leave();
		m_SendComplete.Wait();
		m_CriticalSection.Enter();
		m_SendComplete.Reset();

		// make sure no more TLS threads are added while we are setting stuff up
		m_TLSListCriticalSection.Enter();

		// lock the session info buffers of all threads
		for(FrameProTLS* p_tls = m_FrameProTLSList.GetHead(); p_tls!=NULL; p_tls=p_tls->GetNext())
		{
			if (!p_tls->OnConnected(recording_to_file))		// in case any threads have been added since sending the connect packet
				return false;

			p_tls->LockSessionInfoBuffer();
		}

		// send the session info buffers for all threads
		for(FrameProTLS* p_tls = m_FrameProTLSList.GetHead(); p_tls!=NULL; p_tls=p_tls->GetNext())
			p_tls->SendSessionInfoBuffer();

		// start the new frame
		int64 frame_start_time;
		FRAMEPRO_GET_CLOCK_COUNT(frame_start_time);
		FrameStartPacket frame_start_packet(frame_start_time, 0);
		SendOnMainThread(frame_start_packet);

		if(g_ConditionalScopeMinTime == UINT_MAX)
			g_ConditionalScopeMinTime = (unsigned int)((((int64)FRAMEPRO_DEFAULT_COND_SCOPE_MIN_TIME) * m_ClockFrequency) / 1000000LL);

		// Do this (almost) last. Threads will start sending data once this is set. This atomic flag also publishes all the above data.
		std::atomic_thread_fence(std::memory_order_seq_cst);
		g_Connected = true;

#if FRAMEPRO_SOCKETS_ENABLED
		// create the receive thread. Must do this AFTER setting g_Connected
		if (!m_RecordingFile.IsOpened())
			CreateReceiveThread();
#endif

		// unlock the session info buffers of all threads. Must do this after g_Connected is set
		for(FrameProTLS* p_tls = m_FrameProTLSList.GetHead(); p_tls!=NULL; p_tls=p_tls->GetNext())
			p_tls->UnlockSessionInfoBuffer();

		m_TLSListCriticalSection.Leave();
		
		// start recording context switches if dumping to a file
		if(m_StartContextSwitchRecording)
		{
			StartRecordingContextSitches();
			m_StartContextSwitchRecording = false;
		}

		ClearCustomStatTimers();

		for (CustomStatTimer* p_custom_stat_timer = m_CustomStatTimers; p_custom_stat_timer; p_custom_stat_timer = p_custom_stat_timer->GetNext())
			SetCustomStatUnit(FramePro::RegisterString(p_custom_stat_timer->GetName()), FramePro::RegisterString("cycles"));

		SendScopeColours();
		SendCustomStatGraphs();
		SendCustomStatUnits();
		SendCustomStatColours();

		OnConnectionChanged(true, m_RecordingFile.GetFilename());

		// send date
		m_CriticalSection.Leave();
		StringId date_name = RegisterString("Date");
		char date_str[64];
		GetDateString(date_str, sizeof(date_str));
		StringId date_value = RegisterString(date_str);
		p_framepro_tls->SendPacket(SessionInfoPacket(date_name, date_value));
		m_CriticalSection.Enter();

#if FRAMEPRO_ENABLE_CALLSTACKS
		bool enable_callstacks = m_StartRecordingCallstacks;
		m_StartRecordingCallstacks = false;
		m_CriticalSection.Leave();
		SetCallstacksEnabled(enable_callstacks);
		m_CriticalSection.Enter();
#endif

		return true;
	}

	//------------------------------------------------------------------------
	void FrameProSession::Initialise(FrameProTLS* p_framepro_tls)
	{
		if(m_Initialised)
			return;

		if(!HasSetThreadName(p_framepro_tls->GetThreadId()))
			p_framepro_tls->SetThreadName(p_framepro_tls->GetThreadId(), "Main Thread");

		{
			CriticalSectionScope tls_lock(m_MainThreadSendBufferLock);
			for(Buffer& buffer : m_MainThreadSendBuffer)
				buffer.SetAllocator(GetAllocator());
		}

		{
			CriticalSectionScope tls_lock(m_StringRequestPacketsLock);
			m_StringRequestPackets.SetAllocator(GetAllocator());
		}

		m_ModulePackets.SetAllocator(GetAllocator());
		m_ScopeColours.SetAllocator(GetAllocator());
		m_CustomStatGraphs.SetAllocator(GetAllocator());
		m_CustomStatUnits.SetAllocator(GetAllocator());
		m_CustomStatColours.SetAllocator(GetAllocator());

#if FRAMEPRO_SOCKETS_ENABLED
		OpenListenSocket();
		StartConnectThread();
#endif
		m_Initialised = true;
	}

	//------------------------------------------------------------------------
#if FRAMEPRO_SOCKETS_ENABLED
	void FrameProSession::OpenListenSocket()
	{
		if(m_SocketsBlocked)
			return;

		bool bind_result = m_ListenSocket.Bind(m_Port);

		if(!bind_result)
		{
			DebugWrite("FramePro ERROR: Failed to bind port. This usually means that another process is already running with FramePro enabled.\n");
			return;
		}

		if(!m_ListenSocket.StartListening())
		{
			DebugWrite("FramePro ERROR: Failed to start listening on socket\n");
		}
	}
#endif

	//------------------------------------------------------------------------
#if FRAMEPRO_SOCKETS_ENABLED
	void FrameProSession::StartConnectThread()
	{
		m_ConnectThread.CreateThread(StaticConnectThreadMain, this, GetAllocator());
	}
#endif

	//------------------------------------------------------------------------
	bool FrameProSession::Send(const void* p_data, int size)
	{
		if (m_RecordingFile.IsOpened())
		{
			CriticalSectionScope lock(m_CriticalSection);
			m_RecordingFile.Write(p_data, size);
			m_RecordingFileSize += size;
		}
		else
		{
#if FRAMEPRO_SOCKETS_ENABLED
			if (m_Interactive)
			{
				#if FRAMEPRO_DEBUG_TCP
					static File file;
					if (!file.IsOpened())
					{
						bool opened = file.OpenForWrite("framepro_network_data.framepro_recording");
						FRAMEPRO_ASSERT(opened);
						FRAMEPRO_UNREFERENCED(opened);
					}
					file.Write(p_data, size);
				#endif

				if(!m_ClientSocket.Send(p_data, size))
					return false;	// disconnected
			}
			else
			{
				m_NonInteractiveRecordingFile.Write(p_data, size);
				m_NonInteractiveRecordingFileSize += size;
			}
#endif
		}

		return true;
	}

	//------------------------------------------------------------------------
	void FrameProSession::SendFrameBuffer()
	{
		CriticalSectionScope lock(m_SendFrameBufferCriticalSection);

		List<SendBuffer> send_buffer_list;

		// send the main thread send buffer
		Buffer* p_main_thread_send_buffer = nullptr;
		{
			CriticalSectionScope tls_lock(m_MainThreadSendBufferLock);
			p_main_thread_send_buffer = m_MainThreadSendBuffer + m_MainThreadSendBufferIndex;
			m_MainThreadSendBufferIndex = 1 - m_MainThreadSendBufferIndex;
			m_MainThreadSendBuffer[m_MainThreadSendBufferIndex].Clear();
		}
		
		Send(p_main_thread_send_buffer->GetBuffer(), p_main_thread_send_buffer->GetSize());
		p_main_thread_send_buffer->Clear();

		// get all of the send buffers
		{
			CriticalSectionScope tls_lock(m_TLSListCriticalSection);
			for(FrameProTLS* p_framepro_tls = m_FrameProTLSList.GetHead(); p_framepro_tls!=NULL; p_framepro_tls=p_framepro_tls->GetNext())
				p_framepro_tls->CollectSendBuffers(send_buffer_list);
		}

		// send the send buffers
		for(SendBuffer* p_send_buffer = send_buffer_list.GetHead(); p_send_buffer; p_send_buffer = p_send_buffer->GetNext())
		{
			if(!Send(p_send_buffer->GetBuffer(), p_send_buffer->GetSize()))
				break;		// disconnected
		}

		// give the empty send buffers back to the TLS objects
		SendBuffer* p_iter = send_buffer_list.GetHead();
		while(p_iter)
		{
			SendBuffer* p_next = p_iter->GetNext();

			p_iter->SetNext(NULL);
			p_iter->ClearSize();

			p_iter->GetOwner()->AddEmptySendBuffer(p_iter);

			p_iter = p_next;
		}

		{
			CriticalSectionScope tls_lock(m_TLSListCriticalSection);
			FrameProTLS* p_framepro_tls = m_FrameProTLSList.GetHead();
			while(p_framepro_tls)
			{
				FrameProTLS* p_next = p_framepro_tls->GetNext();

				if (p_framepro_tls->ShuttingDown())
				{
					m_TLSListCriticalSection.Leave();
					DestroyFrameProTLS(p_framepro_tls);
					m_TLSListCriticalSection.Enter();
				}

				p_framepro_tls = p_next;
			}
		}
	}

	//------------------------------------------------------------------------
	void FrameProSession::SendRecordedDataAndDisconnect()
	{
#if FRAMEPRO_SOCKETS_ENABLED
		CriticalSectionScope lock(m_SendFrameBufferCriticalSection);
		CriticalSectionScope lock2(m_CriticalSection);

		FRAMEPRO_ASSERT(!m_Interactive);

		g_Connected = false;

		m_NonInteractiveRecordingFile.Close();

		char folder[FRAMEPRO_MAX_PATH];
		Platform::GetRecordingFolder(folder, sizeof(folder));
		char path[FRAMEPRO_MAX_PATH];
		SPrintf(path, sizeof(path), "%s%s", folder, g_NonInteractiveRecordingFilePath);

		File read_file;
		
		bool opened = read_file.OpenForRead(path);
		FRAMEPRO_ASSERT(opened);
		FRAMEPRO_UNREFERENCED(opened);

		size_t bytes_to_read = read_file.GetSize();

		const int block_size = 64*1024;

		char* p_read_buffer = (char*)m_Allocator.Alloc(block_size);
		if (!p_read_buffer)
			return;

		while(bytes_to_read)
		{
			size_t size_to_read = block_size < bytes_to_read ? block_size : bytes_to_read;
			read_file.Read(p_read_buffer, size_to_read);
			m_ClientSocket.Send(p_read_buffer, size_to_read);
			bytes_to_read -= size_to_read;
		}
		read_file.Close();
		m_Allocator.Free(p_read_buffer);

		Disconnect_NoLock();
#endif
	}

	//------------------------------------------------------------------------
	void FrameProSession::Disconnect(bool wait_for_threads_to_exit)
	{
		CriticalSectionScope lock(m_CriticalSection);
		if(g_Connected)
			Disconnect_NoLock(wait_for_threads_to_exit);
	}

	//------------------------------------------------------------------------
	void FrameProSession::Disconnect_NoLock(bool wait_for_threads_to_exit)
	{
		if(mp_ContextSwitchRecorder)
			Platform::StopRecordingContextSitches(mp_ContextSwitchRecorder);

#if FRAMEPRO_SOCKETS_ENABLED
		m_ClientSocket.Disconnect();
#endif
		g_Connected = false;

		if (wait_for_threads_to_exit)
		{
			// shut down the send thread
			if (m_SendThread.IsAlive())
			{
				m_SendThreadExit = true;
				m_SendReady.Set();
				m_CriticalSection.Leave();
				m_SendThreadFinished.Wait();
				m_CriticalSection.Enter();
				m_SendThreadExit = false;
			}

			// shut down the receive thread
			if (m_ReceiveThread.IsAlive())
			{
				m_CriticalSection.Leave();
				m_ReceiveThreadTerminatedEvent.Wait(10 * 1000);
				m_CriticalSection.Enter();
			}
		}

		{
			CriticalSectionScope tls_lock(m_TLSListCriticalSection);
			for(FrameProTLS* p_framepro_tls = m_FrameProTLSList.GetHead(); p_framepro_tls!=NULL; p_framepro_tls=p_framepro_tls->GetNext())
				p_framepro_tls->OnDisconnected();
		}

		g_ConditionalScopeMinTime = UINT_MAX;

		m_InitialiseConnectionNextFrame = false;

		DynamicWString recording_filename;
		recording_filename.SetAllocator(&m_Allocator);

		if(m_RecordingFile.IsOpened())
		{
			recording_filename = m_RecordingFile.GetFilename().c_str();
			m_RecordingFile.Close();
		}

#if FRAMEPRO_SOCKETS_ENABLED
		// start listening for new connections
		StartConnectThread();
#endif
		OnConnectionChanged(false, recording_filename);
	}

	//------------------------------------------------------------------------
	void FrameProSession::SendHeartbeatInfo(FrameProTLS* p_framepro_tls)
	{
		int64 now;
		FRAMEPRO_GET_CLOCK_COUNT(now);

		if(now - m_LastSessionInfoSendTime > m_ClockFrequency && g_Connected)
		{
			m_LastSessionInfoSendTime = now;

			// notify FramePro of the main thread
			int thread_id = p_framepro_tls->GetThreadId();
			if(m_MainThreadId != thread_id)
			{
				p_framepro_tls->SetMainThread(thread_id);
				m_MainThreadId = thread_id;
			}

			// send session info
			SessionStatsPacket session_stats_packet;

			{
				CriticalSectionScope tls_lock(m_TLSListCriticalSection);
				for(FrameProTLS* p_framepro_tls_iter = m_FrameProTLSList.GetHead(); p_framepro_tls_iter!=NULL; p_framepro_tls_iter=p_framepro_tls_iter->GetNext())
				{
					session_stats_packet.m_SendBufferSize += p_framepro_tls_iter->GetSendBufferMemorySize();
					session_stats_packet.m_StringMemorySize += p_framepro_tls_iter->GetStringMemorySize();
					session_stats_packet.m_MiscMemorySize += sizeof(FrameProTLS);
				}
			}

			session_stats_packet.m_RecordingFileSize = m_NonInteractiveRecordingFileSize;

			SendImmediate(&session_stats_packet, sizeof(session_stats_packet), p_framepro_tls);
		}
	}

	//------------------------------------------------------------------------
	// when interactive mode is disabled sends over the socket directly, otherwise send as normal
	void FrameProSession::SendImmediate(void* p_data, int size, FrameProTLS* p_framepro_tls)
	{
		if(m_RecordingFile.IsOpened())
		{
			p_framepro_tls->Send(p_data, size);
		}
		else
		{
#if FRAMEPRO_SOCKETS_ENABLED
			if(m_Interactive)
				p_framepro_tls->Send(p_data, size);
			else
				m_ClientSocket.Send(p_data, size);
#endif
		}
	}

	//------------------------------------------------------------------------
	void FrameProSession::SendSessionInfo(const char* p_name, const char* p_value)
	{
		StringId name = RegisterString(p_name);
		StringId value = RegisterString(p_value);

		SendSessionInfo(name, value);
	}

	//------------------------------------------------------------------------
	void FrameProSession::SendSessionInfo(const wchar_t* p_name, const wchar_t* p_value)
	{
		StringId name = RegisterString(p_name);
		StringId value = RegisterString(p_value);

		SendSessionInfo(name, value);
	}

	//------------------------------------------------------------------------
	void FrameProSession::SendSessionInfo(StringId name, StringId value)
	{
		// this needs to be outside the critical section lock because it might lock the critical section itself
		FrameProTLS* p_framepro_tls = GetFrameProTLS();

		CriticalSectionScope lock(m_CriticalSection);

		Initialise(p_framepro_tls);

		p_framepro_tls->SendSessionInfoPacket(SessionInfoPacket(name, value));
	}

	//------------------------------------------------------------------------
	void FrameProSession::BlockSockets()
	{
		CriticalSectionScope lock(m_CriticalSection);

		if (!m_SocketsBlocked)
		{
#if FRAMEPRO_SOCKETS_ENABLED
			m_ListenSocket.Disconnect();
#endif
			m_SocketsBlocked = true;
		}
	}

	//------------------------------------------------------------------------
	void FrameProSession::UnblockSockets()
	{
		CriticalSectionScope lock(m_CriticalSection);

		if(m_SocketsBlocked)
		{
			m_SocketsBlocked = false;

			if(m_Initialised)
			{
				#if FRAMEPRO_SOCKETS_ENABLED
					OpenListenSocket();
					StartConnectThread();
				#endif
			}
		}
	}

	//------------------------------------------------------------------------
	void FrameProSession::Shutdown()
	{
		m_TLSListCriticalSection.Enter();

		while (!m_FrameProTLSList.IsEmpty())
		{
			FrameProTLS* p_framepro_tls = m_FrameProTLSList.GetHead();

			m_TLSListCriticalSection.Leave();
			DestroyFrameProTLS(p_framepro_tls);
			m_TLSListCriticalSection.Enter();
		}

		m_TLSListCriticalSection.Leave();
	}

	//------------------------------------------------------------------------
	int64 FrameProSession::GetClockFrequency()
	{
		return m_ClockFrequency;
	}

	//------------------------------------------------------------------------
	void FrameProSession::FrameStart()
	{
		FRAMEPRO_NAMED_SCOPE("FramePro Start Frame");

		// this needs to be outside the critical section lock because it might lock the critical section itself
		FrameProTLS* p_framepro_tls = GetFrameProTLS();

		CriticalSectionScope lock(m_CriticalSection);

		Initialise(p_framepro_tls);

		// initialise the connection
		if(m_InitialiseConnectionNextFrame)
		{
			if (!InitialiseConnection(p_framepro_tls))
			{
				Disconnect_NoLock(false);
				return;
			}

			m_InitialiseConnectionNextFrame = false;
		}

		// send any outstanding string literals
		{
			CriticalSectionScope tls_lock(m_StringRequestPacketsLock);
			int count = m_StringRequestPackets.GetCount();
			if(count)
			{
				for(int i=0; i<count; ++i)
				{
					const RequestStringLiteralPacket& packet = m_StringRequestPackets[i];
					p_framepro_tls->SendStringLiteral((StringLiteralType::Enum)packet.m_StringLiteralType, packet.m_StringId);
				}
				m_StringRequestPackets.Clear();
			}
		}

		if(g_Connected)
		{
			Platform::FlushContextSwitches(mp_ContextSwitchRecorder);

			int64 wait_start_time;
			FRAMEPRO_GET_CLOCK_COUNT(wait_start_time);

			FlushCustomStatTimers(p_framepro_tls);

			{
				FRAMEPRO_NAMED_SCOPE("FramePro Wait For Send");

				if(GetMemoryUsage() > FRAMEPRO_MAX_MEMORY)
				{
					// wait until the send from the previous frame has finished
					m_CriticalSection.Leave();

					m_SendReady.Set();

					m_SendComplete.Wait();

					m_CriticalSection.Enter();
				}
			}

			int64 wait_end_time;
			FRAMEPRO_GET_CLOCK_COUNT(wait_end_time);
			int64 wait_for_send_complete_time = wait_end_time - wait_start_time;

			m_SendComplete.Reset();

			// tell the TLS objects that the frame has started
			bool error = false;
			{
				CriticalSectionScope tls_lock(m_TLSListCriticalSection);
				for (FrameProTLS* p_iter = m_FrameProTLSList.GetHead(); p_iter != NULL; p_iter = p_iter->GetNext())
				{
					if (!p_iter->OnFrameStart())
					{
						error = true;
						break;
					}
				}
			}

			if (error)
			{
				Disconnect_NoLock(false);
				return;
			}

			// send the frame data
			SendHeartbeatInfo(p_framepro_tls);

			int64 frame_start_time;
			FRAMEPRO_GET_CLOCK_COUNT(frame_start_time);
			FrameStartPacket frame_start_packet(frame_start_time, wait_for_send_complete_time);
			SendOnMainThread(frame_start_packet);
		}

		// stop recording if the file has become too big
		#if LIMIT_RECORDING_FILE_SIZE
			if (m_MaxRecordingFileSize &&
				m_RecordingFile.IsOpened() &&
				m_RecordingFileSize > m_MaxRecordingFileSize)
			{
				StopRecording_NoLock();
			}
		#endif
	}

	//------------------------------------------------------------------------
	size_t FrameProSession::GetMemoryUsage() const
	{
		size_t memory = 0;

		CriticalSectionScope tls_lock(m_TLSListCriticalSection);
		for(const FrameProTLS* p_framepro_tls_iter = m_FrameProTLSList.GetHead(); p_framepro_tls_iter!=NULL; p_framepro_tls_iter=p_framepro_tls_iter->GetNext())
		{
			memory += p_framepro_tls_iter->GetSendBufferMemorySize();
			memory += p_framepro_tls_iter->GetStringMemorySize();
			memory += sizeof(FrameProTLS);
		}

		return memory;
	}

	//------------------------------------------------------------------------
	bool FrameProSession::AddFrameProTLS(FrameProTLS* p_framepro_tls)
	{
		CriticalSectionScope lock(m_CriticalSection);

		{
			CriticalSectionScope tls_lock(m_TLSListCriticalSection);
			m_FrameProTLSList.AddTail(p_framepro_tls);
		}

		if (g_Connected)
		{
			if (!p_framepro_tls->OnConnected(m_RecordingFile.IsOpened()))
				return false;
		}

		return true;
	}

	//------------------------------------------------------------------------
	void FrameProSession::RemoveFrameProTLS(FrameProTLS* p_framepro_tls)
	{
		CriticalSectionScope lock(m_CriticalSection);

		{
			CriticalSectionScope tls_lock(m_TLSListCriticalSection);
			m_FrameProTLSList.Remove(p_framepro_tls);
		}
	}

	//------------------------------------------------------------------------
	void FrameProSession::SetThreadName(const char* p_name)
	{
		FrameProTLS* p_framepro_tls = GetFrameProTLS();

		CriticalSectionScope lock(m_CriticalSection);

		m_NamedThreads.SetAllocator(GetAllocator());

		int thread_id = p_framepro_tls->GetThreadId();

		if(!m_NamedThreads.Contains(thread_id))
			m_NamedThreads.Add(thread_id);

		p_framepro_tls->SetThreadName(thread_id, p_name);
	}

	//------------------------------------------------------------------------
	void FrameProSession::SetThreadName(int thread_id, const char* p_name)
	{
		FrameProTLS* p_framepro_tls = GetFrameProTLS();

		CriticalSectionScope lock(m_CriticalSection);

		m_NamedThreads.SetAllocator(GetAllocator());

		if (!m_NamedThreads.Contains(thread_id))
			m_NamedThreads.Add(thread_id);

		p_framepro_tls->SetThreadName(thread_id, p_name);
	}

	//------------------------------------------------------------------------
	bool FrameProSession::HasSetThreadName(int thread_id) const
	{
		return m_NamedThreads.Contains(thread_id);
	}

	//------------------------------------------------------------------------
	int FrameProSession::GetConnectionChangedCallbackIndex(ConnectionChangedCallback p_callback)
	{
		for(int i=0; i<m_Connectionchangedcallbacks.GetCount(); ++i)
		{
			if(m_Connectionchangedcallbacks[i].mp_Callback == p_callback)
				return i;
		}

		return -1;
	}

	//------------------------------------------------------------------------
	void FrameProSession::RegisterConnectionChangedCallback(ConnectionChangedCallback p_callback, void* p_context)
	{
		CriticalSectionScope lock(m_ConnectionChangedCriticalSection);

		// call immediately if already connected
		if(g_Connected)
			p_callback(true, m_RecordingFile.GetFilename().c_str(), p_context);

		if(GetConnectionChangedCallbackIndex(p_callback) == -1)
		{
			ConnectionChangedcallbackInfo data;
			data.mp_Callback = p_callback;
			data.mp_Context = p_context;

			m_Connectionchangedcallbacks.SetAllocator(GetAllocator());

			m_Connectionchangedcallbacks.Add(data);
		}
	}

	//------------------------------------------------------------------------
	void FrameProSession::UnregisterConnectionChangedCallback(ConnectionChangedCallback p_callback)
	{
		CriticalSectionScope lock(m_ConnectionChangedCriticalSection);

		int index = GetConnectionChangedCallbackIndex(p_callback);
		if(index != -1)
			m_Connectionchangedcallbacks.RemoveAt(index);
	}

	//------------------------------------------------------------------------
	void FrameProSession::OnConnectionChanged(bool connected, const DynamicWString& filename) const
	{
		CriticalSectionScope lock(m_ConnectionChangedCriticalSection);

		for(int i=0; i<m_Connectionchangedcallbacks.GetCount(); ++i)
		{
			const ConnectionChangedcallbackInfo& data = m_Connectionchangedcallbacks[i];
			data.mp_Callback(connected, filename.c_str(), data.mp_Context);
		}
	}

	//------------------------------------------------------------------------
	void FrameProSession::StartRecording(const char* p_filename, bool context_switches, bool callstacks, int64 max_file_size)
	{
		Disconnect(true);

		CreateDefaultAllocator();

		CriticalSectionScope lock(m_CriticalSection);

		if(m_RecordingFile.IsOpened())
			StopRecording();

		bool opened_recording_file = m_RecordingFile.OpenForWrite(p_filename);

		if(opened_recording_file)
		{
			const char* p_id = "framepro_recording";
			m_RecordingFile.Write(p_id, strlen(p_id));

			#if FRAMEPRO_SOCKETS_ENABLED
				m_ListenSocket.Disconnect();		// don't allow connections while recording
			#endif
			
			m_StartContextSwitchRecording = context_switches;

			#if FRAMEPRO_ENABLE_CALLSTACKS
				m_StartRecordingCallstacks = callstacks;
			#else
				FRAMEPRO_ASSERT(!callstacks);	// please define FRAMEPRO_ENABLE_CALLSTACKS to enable callstack recording
			#endif

			m_InitialiseConnectionNextFrame = true;

			m_RecordingFileSize = 0;
			m_MaxRecordingFileSize = max_file_size;
		}
		else
		{
			Platform::DebugWrite("FramePro ERROR: Failed to open recording file!");
		}
	}

	//------------------------------------------------------------------------
	void FrameProSession::StartRecording(const wchar_t* p_filename, bool context_switches, bool callstacks, int64 max_file_size)
	{
		Disconnect(true);

		CreateDefaultAllocator();

		CriticalSectionScope lock(m_CriticalSection);

		if(m_RecordingFile.IsOpened())
			StopRecording();

		bool opened_recording_file = m_RecordingFile.OpenForWrite(p_filename);
		FRAMEPRO_ASSERT(opened_recording_file);		// Failed to open recording file

		if(opened_recording_file)
		{
			#if FRAMEPRO_SOCKETS_ENABLED
				m_ListenSocket.Disconnect();		// don't allow connections while recording
			#endif
			
			m_StartContextSwitchRecording = context_switches;

			#if FRAMEPRO_ENABLE_CALLSTACKS
				m_StartRecordingCallstacks = callstacks;
			#else
				FRAMEPRO_ASSERT(!callstacks);	// please define FRAMEPRO_ENABLE_CALLSTACKS to enable callstack recording
			#endif

			m_InitialiseConnectionNextFrame = true;

			m_RecordingFileSize = 0;
			m_MaxRecordingFileSize = max_file_size;
		}
	}

	//------------------------------------------------------------------------
	void FrameProSession::StopRecording()
	{
		CriticalSectionScope lock(m_CriticalSection);

		StopRecording_NoLock();
	}

	//------------------------------------------------------------------------
	void FrameProSession::StopRecording_NoLock()
	{
		if(m_RecordingFile.IsOpened())
		{
			#if FRAMEPRO_SOCKETS_ENABLED
				OpenListenSocket();					// start the listening socket again so that we can accept new connections
			#endif

			Disconnect_NoLock();
		}
	}

	//------------------------------------------------------------------------
	void FrameProSession::SetThreadPriority(int priority)
	{
		m_ThreadPriority = priority;
		m_ThreadPrioritySet = true;

		if(m_SendThread.IsAlive())
			m_SendThread.SetPriority(priority);

		if(m_ReceiveThread.IsAlive())
			m_ReceiveThread.SetPriority(priority);
	}

	//------------------------------------------------------------------------
	void FrameProSession::SetThreadAffinity(int affinity)
	{
		m_ThreadAffinity = affinity;
		m_ThreadAffinitySet = true;

		if(m_SendThread.IsAlive())
			m_SendThread.SetAffinity(affinity);

		if(m_ReceiveThread.IsAlive())
			m_ReceiveThread.SetAffinity(affinity);
	}

	//------------------------------------------------------------------------
	void FrameProSession::AddCustomStatTimer(CustomStatTimer* p_timer)
	{
		{
			CriticalSectionScope lock(m_CriticalSection);

			p_timer->SetNext(m_CustomStatTimers);
			m_CustomStatTimers = p_timer;
		}

		if(g_Connected)
			SetCustomStatUnit(FramePro::RegisterString(p_timer->GetName()), FramePro::RegisterString("cycles"));
	}

	//------------------------------------------------------------------------
	void FrameProSession::FlushCustomStatTimers(FrameProTLS* p_framepro_tls)
	{
		for(CustomStatTimer* p_timer = m_CustomStatTimers; p_timer != NULL; p_timer = p_timer->GetNext())
		{
			uint64 value;
			uint count;
			p_timer->GetAndClear(value, count);

			// if we are connecting to FramePro, FramePro will ask for the string value later, otherwise we need to send it now
			if (p_framepro_tls->SendStringsImmediately())
				p_framepro_tls->SendString(p_timer->GetName(), PacketType::StringPacket);

			CriticalSectionScope lock(p_framepro_tls->GetCurrentSendBufferCriticalSection());

			CustomStatPacketInt64* p_packet = p_framepro_tls->AllocateSpaceInBuffer<CustomStatPacketInt64>();

			CustomStatValueType::Enum value_type = CustomStatValueType::Int64;

			p_packet->m_PacketTypeAndValueType = PacketType::CustomStatPacket | (value_type << 16);
			p_packet->m_Count = count;
			p_packet->m_Name = (StringId)p_timer->GetName();
			p_packet->m_Value = value;
			FRAMEPRO_GET_CLOCK_COUNT(p_packet->m_Time);
		}
	}

	//------------------------------------------------------------------------
	void FrameProSession::ClearCustomStatTimers()
	{
		for (CustomStatTimer* p_timer = m_CustomStatTimers; p_timer != NULL; p_timer = p_timer->GetNext())
		{
			uint64 value;
			uint count;
			p_timer->GetAndClear(value, count);
		}
	}

	//------------------------------------------------------------------------
	bool FrameProSession::CallConditionalParentScopeCallback(ConditionalParentScopeCallback p_callback, const char* p_name, int64 start_time, int64 end_time)
	{
		return p_callback(p_name, start_time, end_time, m_ClockFrequency);
	}

	//------------------------------------------------------------------------
	void FrameProSession::SetConditionalScopeMinTimeInMicroseconds(int64 value)
	{
		g_ConditionalScopeMinTime = (int)((value * m_ClockFrequency) / 1000000LL);
	}

	//------------------------------------------------------------------------
	void FrameProSession::SetScopeColour(StringId name, uint colour)
	{
		CriticalSectionScope lock(m_ScopeColoursLock);

		bool updated_existing = false;
		int count = m_ScopeColours.GetCount();
		for (int i = 0; i < count; ++i)
		{
			if (m_ScopeColours[i].m_Name == name)
			{
				if(m_ScopeColours[i].m_Colour == colour)
					return;

				m_ScopeColours[i].m_Colour = colour;
				updated_existing = true;
				break;
			}
		}

		if (!updated_existing)
		{
			ScopeColour scope_colour;
			scope_colour.m_Name = name;
			scope_colour.m_Colour = colour;
			m_ScopeColours.Add(scope_colour);
		}

		if (g_Connected)
			GetFrameProTLS()->SendScopeColourPacket(name, colour);
	}

	//------------------------------------------------------------------------
	void FrameProSession::SendScopeColours()
	{
		FrameProTLS* p_framepro_tls = GetFrameProTLS();

		CriticalSectionScope lock(m_ScopeColoursLock);

		FRAMEPRO_ASSERT(g_Connected);

		int count = m_ScopeColours.GetCount();
		for (int i = 0; i < count; ++i)
			p_framepro_tls->SendScopeColourPacket(m_ScopeColours[i].m_Name, m_ScopeColours[i].m_Colour);
	}

	//------------------------------------------------------------------------
	void FrameProSession::SetCustomStatGraph(StringId name, StringId graph)
	{
		// this needs to be outside the critical section lock because it might lock the critical section itself
		FrameProTLS* p_framepro_tls = GetFrameProTLS();

		CriticalSectionScope lock(m_CustomStatInfoLock);

		Initialise(p_framepro_tls);

		bool updated_existing = false;
		int count = m_CustomStatGraphs.GetCount();
		for (int i = 0; i < count; ++i)
		{
			if (m_CustomStatGraphs[i].m_Name == name)
			{
				m_CustomStatGraphs[i].m_Value = graph;
				updated_existing = true;
				break;
			}
		}

		if (!updated_existing)
		{
			CustomStatInfo custom_stat_info;
			custom_stat_info.m_Name = name;
			custom_stat_info.m_Value = graph;
			m_CustomStatGraphs.Add(custom_stat_info);
		}

		if (g_Connected)
			GetFrameProTLS()->SendCustomStatGraphPacket(name, graph);
	}

	//------------------------------------------------------------------------
	void FrameProSession::SetCustomStatUnit(StringId name, StringId unit)
	{
		// this needs to be outside the critical section lock because it might lock the critical section itself
		FrameProTLS* p_framepro_tls = GetFrameProTLS();

		CriticalSectionScope lock(m_CustomStatInfoLock);

		Initialise(p_framepro_tls);

		bool updated_existing = false;
		int count = m_CustomStatUnits.GetCount();
		for (int i = 0; i < count; ++i)
		{
			if (m_CustomStatUnits[i].m_Name == name)
			{
				m_CustomStatUnits[i].m_Value = unit;
				updated_existing = true;
				break;
			}
		}

		if (!updated_existing)
		{
			CustomStatInfo custom_stat_info;
			custom_stat_info.m_Name = name;
			custom_stat_info.m_Value = unit;
			m_CustomStatUnits.Add(custom_stat_info);
		}

		if (g_Connected)
			GetFrameProTLS()->SendCustomStatUnitPacket(name, unit);
	}

	//------------------------------------------------------------------------
	void FrameProSession::SetCustomStatColour(StringId name, uint colour)
	{
		// this needs to be outside the critical section lock because it might lock the critical section itself
		FrameProTLS* p_framepro_tls = GetFrameProTLS();

		CriticalSectionScope lock(m_CustomStatInfoLock);

		Initialise(p_framepro_tls);

		bool updated_existing = false;
		int count = m_CustomStatColours.GetCount();
		for (int i = 0; i < count; ++i)
		{
			if (m_CustomStatColours[i].m_Name == name)
			{
				m_CustomStatColours[i].m_Colour = colour;
				updated_existing = true;
				break;
			}
		}

		if (!updated_existing)
		{
			ScopeColour custom_stat_colour;
			custom_stat_colour.m_Name = name;
			custom_stat_colour.m_Colour = colour;
			m_CustomStatColours.Add(custom_stat_colour);
		}

		if (g_Connected)
			GetFrameProTLS()->SendCustomStatColourPacket(name, colour);
	}

	//------------------------------------------------------------------------
	void FrameProSession::SendScopeCallstack()
	{
#if FRAMEPRO_ENABLE_CALLSTACKS
		if (!g_Connected)
			return;

		SendModules();

		FrameProTLS* p_framepro_tls = GetFrameProTLS();

		CallstackResult callstack_result = p_framepro_tls->GetCallstack();

		p_framepro_tls->SetCurrentCallstackId(callstack_result.mp_Callstack->m_ID);

		int send_size = sizeof(CallstackPacket);
		if (callstack_result.m_IsNew)
			send_size += callstack_result.mp_Callstack->m_Size * sizeof(uint64);

		{
			CriticalSectionScope lock(p_framepro_tls->GetCurrentSendBufferCriticalSection());

			CallstackPacket* p_packet = (CallstackPacket*)p_framepro_tls->AllocateSpaceInBuffer(send_size);

			p_packet->m_PacketType = PacketType::CallstackPacket;
			p_packet->m_CallstackId = callstack_result.mp_Callstack->m_ID;
			p_packet->m_CallstackSize = 0;

			if (callstack_result.m_IsNew)
			{
				p_packet->m_CallstackSize = callstack_result.mp_Callstack->m_Size;
				memcpy(
					(char*)(p_packet + 1),
					callstack_result.mp_Callstack->mp_Stack,
					callstack_result.mp_Callstack->m_Size * sizeof(uint64));
			}
		}
#endif
	}

	//------------------------------------------------------------------------
	void FrameProSession::SendCustomStatGraphs()
	{
		FrameProTLS* p_framepro_tls = GetFrameProTLS();

		CriticalSectionScope lock(m_CustomStatInfoLock);

		FRAMEPRO_ASSERT(g_Connected);

		int count = m_CustomStatGraphs.GetCount();
		for (int i = 0; i < count; ++i)
			p_framepro_tls->SendCustomStatGraphPacket(m_CustomStatGraphs[i].m_Name, m_CustomStatGraphs[i].m_Value);
	}

	//------------------------------------------------------------------------
	void FrameProSession::SendCustomStatUnits()
	{
		FrameProTLS* p_framepro_tls = GetFrameProTLS();

		CriticalSectionScope lock(m_CustomStatInfoLock);

		FRAMEPRO_ASSERT(g_Connected);

		int count = m_CustomStatUnits.GetCount();
		for (int i = 0; i < count; ++i)
			p_framepro_tls->SendCustomStatUnitPacket(m_CustomStatUnits[i].m_Name, m_CustomStatUnits[i].m_Value);
	}

	//------------------------------------------------------------------------
	void FrameProSession::SendCustomStatColours()
	{
		FrameProTLS* p_framepro_tls = GetFrameProTLS();

		CriticalSectionScope lock(m_CustomStatInfoLock);

		FRAMEPRO_ASSERT(g_Connected);

		int count = m_CustomStatColours.GetCount();
		for (int i = 0; i < count; ++i)
			p_framepro_tls->SendCustomStatColourPacket(m_CustomStatColours[i].m_Name, m_CustomStatColours[i].m_Colour);
	}
}


//------------------------------------------------------------------------
//
// FrameProStackTrace.cpp
//

//------------------------------------------------------------------------
#if FRAMEPRO_ENABLE_CALLSTACKS

//------------------------------------------------------------------------
namespace FramePro
{
	//------------------------------------------------------------------------
	StackTrace::StackTrace(Allocator* p_allocator)
		: m_StackCount(0),
		m_StackHash(0),
		m_CallstackSet(p_allocator)
	{
		memset(m_Stack, 0, sizeof(m_Stack));
	}

	//------------------------------------------------------------------------
	void StackTrace::Clear()
	{
		m_CallstackSet.Clear();
	}

	//------------------------------------------------------------------------
	CallstackResult StackTrace::Capture()
	{
		CallstackResult result;

		result.m_IsNew = false;

		memset(m_Stack, 0, sizeof(m_Stack));

		if (!Platform::GetStackTrace(m_Stack, m_StackCount, m_StackHash))
		{
			result.mp_Callstack = NULL;
			return result;
		}

		result.mp_Callstack = m_CallstackSet.Get((uint64*)m_Stack, m_StackCount, m_StackHash);

		if (!result.mp_Callstack)
		{
			result.mp_Callstack = m_CallstackSet.Add((uint64*)m_Stack, m_StackCount, m_StackHash);
			result.m_IsNew = true;
		}

		return result;
	}
}

//------------------------------------------------------------------------
#endif		// #if FRAMEPRO_ENABLE_CALLSTACKS


//------------------------------------------------------------------------
//
// FrameProTLS.cpp
//

//------------------------------------------------------------------------
#if FRAMEPRO_WIN_BASED_PLATFORM
	#pragma warning(push)
	#pragma warning(disable : 4127)
#endif

//------------------------------------------------------------------------
namespace FramePro
{
	//------------------------------------------------------------------------
	static const int g_FrameProTLSBufferMarker = 0xfbfbfbfb;

	std::atomic<long> FrameProTLS::m_StringCount(0);

	//------------------------------------------------------------------------
	FrameProTLS::FrameProTLS(Allocator* p_allocator, int64 clock_frequency)
	:	m_Interactive(true),
		m_RecordingToFile(false),
		m_SendStringsImmediately(false),
		mp_CurrentSendBuffer(NULL),
		m_CurrentSendBufferSize(0),
		m_ThreadId(Platform::GetCurrentThreadId()),
		m_FirstHiresTimerTime(0),
		m_HiResTimerStartTime(0),
		m_ActiveHiResTimerIndex(-1),
		mp_Next(NULL),
		mp_Allocator(p_allocator),
		m_LiteralStringSet(p_allocator),
		m_LiteralStringSetMemorySize(0),
		m_StringHashMap(p_allocator),
		m_WStringHashMap(p_allocator),
		m_SessionInfoBuffer(p_allocator),
		m_SessionInfoBufferMemorySize(0),
		m_Connected(false),
		m_StringAllocator(p_allocator),
		m_SendBufferMemorySize(0),
		m_StringMemorySize(0),
		m_ClockFrequency(clock_frequency),
		m_ShuttingDown(false),
		mp_CurrentConditionalParentScope(NULL),
		m_CurrentCallstackId(-1),
		m_CurrentCallstackIdTime(0)
#if FRAMEPRO_ENABLE_CALLSTACKS
		,m_StackTrace(p_allocator)
		,m_SendCallstacks(false)
#endif
#if FRAMEPRO_DEBUG || FRAMEPRO_ALLOW_UNPARENTED_HIRES_SCOPES
		,m_ScopeDepth(0)
#endif
	{
		UpdateSendStringsImmediatelyFlag();

		m_InitialisedCustomStats.SetAllocator(p_allocator);

		memset(m_FalseSharingSpacerBuffer, g_FrameProTLSBufferMarker, sizeof(m_FalseSharingSpacerBuffer)); //-V575

		m_HiResTimers.SetAllocator(p_allocator);
		m_PausedHiResTimerStack.SetAllocator(p_allocator);

#ifdef FRAMEPRO_SCOPE_MIN_TIME
		m_ScopeMinTime = FramePro_Max(1LL, (FRAMEPRO_SCOPE_MIN_TIME * m_ClockFrequency) / 1000000000LL);
#endif

#ifdef FRAMEPRO_WAIT_EVENT_MIN_TIME
		m_WaitEventMinTime = FramePro_Max(1LL, (FRAMEPRO_WAIT_EVENT_MIN_TIME * m_ClockFrequency) / 1000000000LL);
#endif
	}

	//------------------------------------------------------------------------
	FrameProTLS::~FrameProTLS()
	{
		{
			CriticalSectionScope lock(m_CriticalSection);
			Clear();
		}

		FreeCurrentSendBuffer();
	}

	//------------------------------------------------------------------------
	// this is called from the main thread
	void FrameProTLS::OnDisconnected()
	{
		CriticalSectionScope lock(m_CriticalSection);

		m_Connected = false;
		SetInteractive(true);		// we are interactive until told otherwise

		Clear();
	}

	//------------------------------------------------------------------------
	void FrameProTLS::Clear()
	{
		FRAMEPRO_ASSERT(m_CriticalSection.Locked());

		DeleteListItems(m_SendBufferList);
		DeleteListItems(m_SendBufferFreeList);

		m_SendBufferMemorySize = 0;

		{
			CriticalSectionScope frame_swap_lock(m_CurrentSendBufferCriticalSection);
			m_CurrentSendBufferSize = 0;
		}

		{
			CriticalSectionScope cond_lock(m_ConditionalParentScopeListCritSec);
			ConditionalParentScope* p_scope = m_ConditionalParentScopeList.GetHead();
			while (p_scope)
			{
				ConditionalParentScope* p_next = p_scope->GetNext();
				DeleteListItems(p_scope->m_ChildSendBuffers);
				Delete(mp_Allocator, p_scope);
				p_scope = p_next;
			}
			m_ConditionalParentScopeList.Clear();
		}

		m_InitialisedCustomStats.Clear();

		UpdateStringMemorySize();

#if FRAMEPRO_ENABLE_CALLSTACKS
		m_StackTrace.Clear();
#endif
		// we can't delete the hires timer stuff here because we don't want to introduce a lock
	}

	//------------------------------------------------------------------------
	void FrameProTLS::UpdateStringMemorySize() 
	{
		m_StringMemorySize =
			m_StringAllocator.GetMemorySize() +
			m_StringHashMap.GetMemorySize() +
			m_WStringHashMap.GetMemorySize();
	}

	//------------------------------------------------------------------------
	// this is called from the main thread
	bool FrameProTLS::OnConnected(bool recording_to_file)
	{
		CriticalSectionScope lock(m_CriticalSection);
		
		if(!m_Connected)
		{
			Clear();

			m_Connected = true;
			
			m_RecordingToFile = recording_to_file;
			UpdateSendStringsImmediatelyFlag();

			{
				CriticalSectionScope lock2(m_CurrentSendBufferCriticalSection);
				if (!AllocateCurrentSendBuffer())
					return false;
			}
		}

		return true;
	}

	//------------------------------------------------------------------------
	bool FrameProTLS::AllocateCurrentSendBuffer()
	{
		FRAMEPRO_ASSERT(m_CriticalSection.Locked());
		FRAMEPRO_ASSERT(m_CurrentSendBufferCriticalSection.Locked());
		FRAMEPRO_ASSERT(IsOnTLSThread() || !g_Connected);		// can only be accessed from TLS thread, unless we haven't connected yet

		if(!mp_CurrentSendBuffer)
		{
			mp_CurrentSendBuffer = mp_Allocator->Alloc(m_SendBufferCapacity);
			if (!mp_CurrentSendBuffer)
				return false;

			m_SendBufferMemorySize = m_SendBufferMemorySize + m_SendBufferCapacity;
		}

		return true;
	}

	//------------------------------------------------------------------------
	void FrameProTLS::FreeCurrentSendBuffer()
	{
		CriticalSectionScope lock(m_CurrentSendBufferCriticalSection);

		if(mp_CurrentSendBuffer)
		{
			mp_Allocator->Free(mp_CurrentSendBuffer);
			mp_CurrentSendBuffer = NULL;
			m_CurrentSendBufferSize = 0;
		}
	}

	//------------------------------------------------------------------------
	// send the session info buffer (stuff we cache between connects)
	void FrameProTLS::SendSessionInfoBuffer()
	{
		// m_SessionInfoBufferLock should have been locked before calling this function
		Send(m_SessionInfoBuffer.GetBuffer(), m_SessionInfoBuffer.GetSize());
	}

	//------------------------------------------------------------------------
	void FrameProTLS::SendConnectPacket(int64 clock_frequency, int process_id, Platform::Enum platform)
	{
		SendPacket(ConnectPacket(clock_frequency, process_id, platform));
		FlushCurrentSendBuffer();
	}

	//------------------------------------------------------------------------
	void FrameProTLS::Send(const void* p_data, int size)
	{
		FRAMEPRO_ASSERT(IsOnTLSThread() || !g_Connected);		// can only be accessed from TLS thread, unless we haven't connected yet

		// make common case fast
		if(size <= m_SendBufferCapacity)
		{
			CriticalSectionScope lock(m_CurrentSendBufferCriticalSection);

			void* p_dest = AllocateSpaceInBuffer(size);
			memcpy(p_dest, p_data, size);
		}
		else
		{
			List<SendBuffer> send_buffer_list;

			{
				CriticalSectionScope lock(m_CurrentSendBufferCriticalSection);

				int available_space = m_SendBufferCapacity - m_CurrentSendBufferSize;

				if (!available_space)
				{
					FlushCurrentSendBuffer_no_lock();
					available_space = m_SendBufferCapacity;
				}

				int bytes_to_send = size;
				const void* p_src = p_data;
				while (bytes_to_send)
				{
					// copy to the current send buffer
					int send_size = FramePro_Min(bytes_to_send, available_space);
					void* p_dest = (char*)mp_CurrentSendBuffer + m_CurrentSendBufferSize;
					memcpy(p_dest, p_src, send_size);
					m_CurrentSendBufferSize += send_size;
					bytes_to_send -= send_size;
					p_src = (char*)p_src + send_size;

					// copy current send buffer to a new send buffer object
					SendBuffer* p_send_buffer = AllocateSendBuffer();
					if (!p_send_buffer)
						return;

					p_send_buffer->Swap(mp_CurrentSendBuffer, m_CurrentSendBufferSize, m_SendBufferCapacity);
					FRAMEPRO_ASSERT(mp_CurrentSendBuffer);
					available_space = m_SendBufferCapacity;

					send_buffer_list.AddTail(p_send_buffer);
				}
			}

			{
				CriticalSectionScope lock(m_CriticalSection);
				m_SendBufferList.MoveAppend(send_buffer_list);
			}
		}
	}

	//------------------------------------------------------------------------
	bool FrameProTLS::OnFrameStart()
	{
		UpdateStringMemorySize();

		m_SessionInfoBufferMemorySize = m_SessionInfoBuffer.GetMemorySize();

		if (!FlushCurrentSendBuffer())
			return false;

		FlushConditionalChildSendBuffers();

		return true;
	}

	//------------------------------------------------------------------------
	void FrameProTLS::AddEmptySendBuffer(SendBuffer* p_send_buffer)
	{
		CriticalSectionScope lock(m_CriticalSection);

		FRAMEPRO_ASSERT(p_send_buffer->GetOwner() == this);

		// only keep the buffer for the first free send buffer, otherwise clear it
		if(m_SendBufferFreeList.IsEmpty())
		{
			m_SendBufferFreeList.AddHead(p_send_buffer);
		}
		else
		{
			FRAMEPRO_ASSERT(m_SendBufferMemorySize >= (size_t)p_send_buffer->GetCapacity());
			m_SendBufferMemorySize = m_SendBufferMemorySize - p_send_buffer->GetCapacity();		// doesn't have to be atomic because it's only for stats

			p_send_buffer->ClearBuffer();

			m_SendBufferFreeList.AddTail(p_send_buffer);
		}
	}

	//------------------------------------------------------------------------
	void FrameProTLS::SetThreadName(int thread_id, const char* p_name)
	{
		StringId name_id = RegisterString(p_name);

		SendSessionInfoPacket(ThreadNamePacket(thread_id, name_id));
	}

	//------------------------------------------------------------------------
	void FrameProTLS::SetThreadOrder(StringId thread_name)
	{
		SendSessionInfoPacket(ThreadOrderPacket(thread_name));
	}

	//------------------------------------------------------------------------
	void FrameProTLS::SetMainThread(int main_thread_id)
	{
		SendSessionInfoPacket(MainThreadPacket(main_thread_id));
	}

	//------------------------------------------------------------------------
	UE_AUTORTFM_ALWAYS_OPEN
	StringId FrameProTLS::RegisterString(const char* p_str)
	{
		FRAMEPRO_ASSERT(IsOnTLSThread());

		String str(p_str);

		StringId string_id = 0;
		if(!m_StringHashMap.TryGetValue(str, string_id))
		{
			string_id = ++m_StringCount;
			str.TakeCopy(m_StringAllocator);
			if (!m_StringHashMap.Add(str, string_id))
				return 0;
		
			SendString(string_id, p_str, PacketType::StringPacket);

			UpdateStringMemorySize();
		}

		return string_id;
	}

	//------------------------------------------------------------------------
	UE_AUTORTFM_ALWAYS_OPEN
	StringId FrameProTLS::RegisterString(const wchar_t* p_str)
	{
		FRAMEPRO_ASSERT(IsOnTLSThread());

		WString str(p_str);

		StringId string_id = 0;
		if(!m_WStringHashMap.TryGetValue(str, string_id))
		{
			string_id = ++m_StringCount;
			str.TakeCopy(m_StringAllocator);
			if (!m_WStringHashMap.Add(str, string_id))
				return 0;
		
			SendString(string_id, p_str, PacketType::WStringPacket);

			UpdateStringMemorySize();
		}

		return string_id;
	}

	//------------------------------------------------------------------------
	void FrameProTLS::SendSessionInfo(const void* p_data, int size)
	{
		{
			// copy it to the session buffer
			CriticalSectionScope lock(m_SessionInfoBufferLock);
			void* p_dest = m_SessionInfoBuffer.Allocate(size);
			if (!p_dest)
				return;
			memcpy(p_dest, p_data, size);
		}

		if(m_Connected)
			Send(p_data, size);
	}

	//------------------------------------------------------------------------
	void FrameProTLS::SendString(StringId string_id, const char* p_str, PacketType::Enum packet_type)
	{
		FRAMEPRO_ASSERT(IsOnTLSThread());

		int string_len = (int)strlen(p_str);
		FRAMEPRO_ASSERT(string_len <= INT_MAX);

		int aligned_string_len = AlignUpPow2(string_len, 4);
		int size_to_allocate = sizeof(StringPacket) + aligned_string_len;
 
		StringPacket* p_packet = NULL;
		{
			CriticalSectionScope lock(m_SessionInfoBufferLock);

			p_packet = (StringPacket*)(m_SessionInfoBuffer.Allocate(size_to_allocate));
			if(!p_packet)
				return;

			p_packet->m_PacketType = packet_type;
			p_packet->m_Length = (int)string_len;
			p_packet->m_StringId = string_id;
			memcpy(p_packet + 1, p_str, string_len);
		}

		if(m_Connected)
			Send(p_packet, size_to_allocate);
	}

	//------------------------------------------------------------------------
	void FrameProTLS::SendString(StringId string_id, const wchar_t* p_str, PacketType::Enum packet_type)
	{
		FRAMEPRO_ASSERT(IsOnTLSThread());

		int string_len = (int)wcslen(p_str);
		FRAMEPRO_ASSERT(string_len <= INT_MAX);

		StringPacket* p_packet = NULL;
		int size_to_allocate = 0;

		if(sizeof(wchar_t) == 2)
		{
			int string_size = string_len * sizeof(wchar_t);
			int aligned_string_size = AlignUpPow2(string_size, 4);
			size_to_allocate = sizeof(StringPacket) + aligned_string_size;

			{
				CriticalSectionScope lock(m_SessionInfoBufferLock);

				p_packet = (StringPacket*)(m_SessionInfoBuffer.Allocate(size_to_allocate));
				if(!p_packet)
					return;

				p_packet->m_PacketType = packet_type;
				p_packet->m_Length = (int)string_len;
				p_packet->m_StringId = string_id;
				memcpy(p_packet + 1, p_str, string_size);
			}
		}
		else
		{
			FRAMEPRO_ASSERT(sizeof(wchar_t) == 4);	// FramePro only supports 2 or 4 byte wchars
			int string_size = string_len * 2;
			int aligned_string_size = AlignUpPow2(string_size, 4);
			size_to_allocate = sizeof(StringPacket) + aligned_string_size;

			{
				CriticalSectionScope lock(m_SessionInfoBufferLock);

				p_packet = (StringPacket*)(m_SessionInfoBuffer.Allocate(size_to_allocate));
				if (!p_packet)
					return;

				p_packet->m_PacketType = packet_type;
				p_packet->m_Length = (int)string_len;
				p_packet->m_StringId = string_id;

				// convert UTF-32 to UTF-16 by truncating (take only first 2 bytes of the 4 bytes)
				char* p_dest = (char*)(p_packet + 1);
				char* p_source = (char*)p_str;
				for (int i = 0; i < string_len; ++i)
				{
					*p_dest++ = *p_source++;
					*p_dest++ = *p_source++;
					p_source += 2;
				}
			}
		}

		if(m_Connected)
			Send(p_packet, size_to_allocate);
	}

	//------------------------------------------------------------------------
	void FrameProTLS::SendLogPacket(const char* p_message)
	{
		FRAMEPRO_ASSERT(IsOnTLSThread());
		FRAMEPRO_ASSERT(m_Connected);

		int string_len = (int)strlen(p_message);
		FRAMEPRO_ASSERT(string_len <= INT_MAX);

		int aligned_string_len = AlignUpPow2(string_len, 4);
		int size_to_allocate = sizeof(LogPacket) + aligned_string_len;

		CriticalSectionScope lock(m_CurrentSendBufferCriticalSection);

		LogPacket* p_packet = (LogPacket*)AllocateSpaceInBuffer(size_to_allocate);

		int64 time;
		FRAMEPRO_GET_CLOCK_COUNT(time);

		p_packet->m_PacketType = PacketType::LogPacket;
		p_packet->m_Time = time;

		p_packet->m_Length = (int)string_len;
		memcpy(p_packet + 1, p_message, string_len);
	}

	//------------------------------------------------------------------------
	void FrameProTLS::SendEventPacket(const char* p_name, uint colour)
	{
		FRAMEPRO_ASSERT(IsOnTLSThread());
		FRAMEPRO_ASSERT(m_Connected);

		int64 timestamp = 0;
		FRAMEPRO_GET_CLOCK_COUNT(timestamp);

		// if we are connecting to FramePro, FramePro will ask for the string value later, otherwise we need to send it now
		if (m_SendStringsImmediately)
			SendString(p_name, PacketType::StringPacket);

		EventPacket packet;

		packet.m_PacketType = PacketType::EventPacket;
		packet.m_Colour = colour;
		packet.m_Name = (StringId)p_name;
		packet.m_Time = timestamp;

		SendPacket(packet);
	}

	//------------------------------------------------------------------------
	void FrameProTLS::SendScopeColourPacket(StringId name, uint colour)
	{
		FRAMEPRO_ASSERT(IsOnTLSThread());
		FRAMEPRO_ASSERT(m_Connected);

		ScopeColourPacket packet;

		packet.m_PacketType = PacketType::ScopeColourPacket;
		packet.m_Colour = colour;
		packet.m_Name = name;

		SendPacket(packet);
	}

	//------------------------------------------------------------------------
	void FrameProTLS::SendCustomStatGraphPacket(StringId name, StringId graph)
	{
		FRAMEPRO_ASSERT(IsOnTLSThread());
		FRAMEPRO_ASSERT(m_Connected);

		CustomStatInfoPacket packet;

		packet.m_PacketType = PacketType::CustomStatGraphPacket;
		packet.m_Name = name;
		packet.m_Value = graph;

		SendPacket(packet);
	}

	//------------------------------------------------------------------------
	void FrameProTLS::SendCustomStatUnitPacket(StringId name, StringId unit)
	{
		FRAMEPRO_ASSERT(IsOnTLSThread());
		FRAMEPRO_ASSERT(m_Connected);

		CustomStatInfoPacket packet;

		packet.m_PacketType = PacketType::CustomStatUnitPacket;
		packet.m_Name = name;
		packet.m_Value = unit;

		SendPacket(packet);
	}

	//------------------------------------------------------------------------
	void FrameProTLS::SendCustomStatColourPacket(StringId name, uint colour)
	{
		FRAMEPRO_ASSERT(IsOnTLSThread());
		FRAMEPRO_ASSERT(m_Connected);

		CustomStatColourPacket packet;

		packet.m_PacketType = PacketType::CustomStatColourPacket;
		packet.m_Colour = colour;
		packet.m_Name = name;

		SendPacket(packet);
	}

	//------------------------------------------------------------------------
	void FrameProTLS::SendString(const char* p_string, PacketType::Enum packet_type)
	{
		FRAMEPRO_ASSERT(IsOnTLSThread());

		if(AddStringLiteral(p_string))
			SendString((StringId)p_string, p_string, packet_type);
	}

	//------------------------------------------------------------------------
	void FrameProTLS::SendString(const wchar_t* p_string, PacketType::Enum packet_type)
	{
		FRAMEPRO_ASSERT(IsOnTLSThread());

		if(AddStringLiteral(p_string))
			SendString((StringId)p_string, p_string, packet_type);
	}

	//------------------------------------------------------------------------
	void FrameProTLS::SendStringLiteral(StringLiteralType::Enum string_literal_type, StringId string_id)
	{
		switch(string_literal_type)
		{
			case StringLiteralType::NameAndSourceInfo:
				SendString(string_id, (const char*)string_id, PacketType::NameAndSourceInfoPacket);
				break;

			case StringLiteralType::NameAndSourceInfoW:
				SendString(string_id, (const wchar_t*)string_id, PacketType::NameAndSourceInfoPacketW);
				break;

			case StringLiteralType::SourceInfo:
				SendString(string_id, (const char*)string_id, PacketType::SourceInfoPacket);
				break;

			case StringLiteralType::GeneralString:
				SendString(string_id, (const char*)string_id, PacketType::StringPacket);
				break;

			case StringLiteralType::GeneralStringW:
				SendString(string_id, (const wchar_t*)string_id, PacketType::WStringPacket);
				break;

			case StringLiteralType::StringLiteralTimerName:
				SendString(string_id, (const char*)string_id, PacketType::StringLiteralTimerNamePacket);
				break;

			default:
				FramePro::DebugBreak();
				break;
		}
	}

	//------------------------------------------------------------------------
	bool FrameProTLS::FlushCurrentSendBuffer()
	{
		SendBuffer* p_send_buffer = AllocateSendBuffer();
		if (!p_send_buffer)
			return false;

		{
			CriticalSectionScope lock(m_CurrentSendBufferCriticalSection);
			p_send_buffer->Swap(mp_CurrentSendBuffer, m_CurrentSendBufferSize, m_SendBufferCapacity);

			FRAMEPRO_ASSERT(mp_CurrentSendBuffer);
			FRAMEPRO_ASSERT(!m_CurrentSendBufferSize);
		}

		if (mp_CurrentConditionalParentScope)
		{
			CriticalSectionScope lock(m_ConditionalParentScopeListCritSec);
			mp_CurrentConditionalParentScope->m_ChildSendBuffers.AddTail(p_send_buffer);
		}
		else
		{
			CriticalSectionScope lock(m_CriticalSection);
			m_SendBufferList.AddTail(p_send_buffer);
		}

		return true;
	}

	//------------------------------------------------------------------------
	void FrameProTLS::FlushCurrentSendBuffer_no_lock()
	{
		FRAMEPRO_ASSERT(m_CurrentSendBufferCriticalSection.Locked());
		FRAMEPRO_ASSERT(IsOnTLSThread() || !g_Connected);		// can only be accessed from TLS thread, unless we haven't connected yet

		SendBuffer* p_send_buffer = AllocateSendBuffer();
		if (!p_send_buffer)
			return;

		p_send_buffer->Swap(mp_CurrentSendBuffer, m_CurrentSendBufferSize, m_SendBufferCapacity);

		FRAMEPRO_ASSERT(mp_CurrentSendBuffer);
		FRAMEPRO_ASSERT(!m_CurrentSendBufferSize);

		if (mp_CurrentConditionalParentScope)
		{
			SendBuffer* p_new_parent_send_buffer = AllocateSendBuffer();
			if (!p_new_parent_send_buffer)
				return;

			{
				// move the current child send buffer to the parents child buffer list
				CriticalSectionScope lock(m_ConditionalParentScopeListCritSec);
				mp_CurrentConditionalParentScope->m_ChildSendBuffers.AddTail(p_send_buffer);

				p_new_parent_send_buffer->Swap(mp_CurrentConditionalParentScope->mp_SendBuffer);
			}

			{
				// move the parent send buffer to the main send buffer list
				CriticalSectionScope lock(m_CriticalSection);
				m_SendBufferList.AddTail(p_new_parent_send_buffer);
			}
		}
		else
		{
			CriticalSectionScope lock(m_CriticalSection);
			m_SendBufferList.AddTail(p_send_buffer);
		}
	}

	//------------------------------------------------------------------------
	SendBuffer* FrameProTLS::AllocateSendBuffer()
	{
		CriticalSectionScope lock(m_CriticalSection);

		SendBuffer* p_send_buffer = NULL;

		if(!m_SendBufferFreeList.IsEmpty())
		{
			p_send_buffer = m_SendBufferFreeList.RemoveHead();
		}
		else
		{
			p_send_buffer = New<SendBuffer>(mp_Allocator, mp_Allocator, m_SendBufferCapacity, this);
			if (!p_send_buffer)
				return NULL;

			m_SendBufferMemorySize = m_SendBufferMemorySize + m_SendBufferCapacity + sizeof(SendBuffer);		// doesn't need to be atomic, it's only for stats
		}

		FRAMEPRO_ASSERT(!p_send_buffer->GetSize());
		FRAMEPRO_ASSERT(!p_send_buffer->GetNext());

		if (!p_send_buffer->GetBuffer())
		{
			if (!p_send_buffer->AllocateBuffer(m_SendBufferCapacity))
				return NULL;

			m_SendBufferMemorySize = m_SendBufferMemorySize + m_SendBufferCapacity;
		}

		p_send_buffer->SetCreationTime();

		return p_send_buffer;
	}

	//------------------------------------------------------------------------
	void FrameProTLS::CollectSendBuffers(List<SendBuffer>& list)
	{
		CriticalSectionScope lock(m_CriticalSection);

		list.MoveAppend(m_SendBufferList);
	}

	//------------------------------------------------------------------------
	ConditionalParentScope* FrameProTLS::GetConditionalParentScope(const char* p_name)
	{
		FRAMEPRO_ASSERT(m_ConditionalParentScopeListCritSec.Locked());

		ConditionalParentScope* p_scope = m_ConditionalParentScopeList.GetHead();
		while (p_scope)
		{
			if (p_scope->mp_Name == p_name)
				return p_scope;

			p_scope = p_scope->GetNext();
		}

		return NULL;
	}

	//------------------------------------------------------------------------
	ConditionalParentScope* FrameProTLS::CreateConditionalParentScope(const char* p_name)
	{
		FRAMEPRO_ASSERT(m_ConditionalParentScopeListCritSec.Locked());

		ConditionalParentScope* p_scope = New<ConditionalParentScope>(mp_Allocator, p_name);
		if (!p_scope)
			return NULL;
	
		m_ConditionalParentScopeList.AddTail(p_scope);

		return p_scope;
	}

	//------------------------------------------------------------------------
	void FrameProTLS::PushConditionalParentScope(const char* p_name, int64 pre_duration, int64 post_duration)
	{
		CriticalSectionScope lock(m_ConditionalParentScopeListCritSec);

		FRAMEPRO_ASSERT(!mp_CurrentConditionalParentScope);		// nested conditional parent scopes not supported

		ConditionalParentScope* p_scope = GetConditionalParentScope(p_name);
		if (!p_scope)
		{
			p_scope = CreateConditionalParentScope(p_name);
			if (!p_scope)
				return;
		}

		FRAMEPRO_ASSERT(!p_scope->mp_SendBuffer);
		p_scope->mp_SendBuffer = AllocateSendBuffer();
		if (!p_scope->mp_SendBuffer)
			return;

		p_scope->m_PreDuration = pre_duration;
		p_scope->m_PostDuration = post_duration;

		{
			CriticalSectionScope send_buffer_lock(m_CurrentSendBufferCriticalSection);
			p_scope->mp_SendBuffer->Swap(mp_CurrentSendBuffer, m_CurrentSendBufferSize, m_SendBufferCapacity);
		}

		mp_CurrentConditionalParentScope = p_scope;
	}

	//------------------------------------------------------------------------
	void FrameProTLS::PopConditionalParentScope(bool add_children)
	{
		CriticalSectionScope lock(m_ConditionalParentScopeListCritSec);

		ConditionalParentScope* p_scope = mp_CurrentConditionalParentScope;
		mp_CurrentConditionalParentScope = NULL;

		FRAMEPRO_ASSERT(p_scope);		// popped without a push

		{
			// restore the original parent send buffer and grab the current one
			CriticalSectionScope send_buffer_lock(m_CurrentSendBufferCriticalSection);
			p_scope->mp_SendBuffer->Swap(mp_CurrentSendBuffer, m_CurrentSendBufferSize, m_SendBufferCapacity);
		}

		p_scope->m_ChildSendBuffers.AddTail(p_scope->mp_SendBuffer);
		p_scope->mp_SendBuffer = NULL;

		if (add_children)
		{
			FRAMEPRO_GET_CLOCK_COUNT(p_scope->m_LastPopConditionalChildrenTime);
		}

		int64 now;
		FRAMEPRO_GET_CLOCK_COUNT(now);
		bool in_post_duration = now - p_scope->m_LastPopConditionalChildrenTime < (p_scope->m_PostDuration * m_ClockFrequency) / 1000000;

		if (add_children || in_post_duration)
		{
			{
				CriticalSectionScope send_lock(m_CriticalSection);
				m_SendBufferList.MoveAppend(p_scope->m_ChildSendBuffers);
			}
		}
	}

	//------------------------------------------------------------------------
	void FrameProTLS::FlushConditionalChildSendBuffers()
	{
		CriticalSectionScope lock(m_ConditionalParentScopeListCritSec);
		
		int64 now;
		FRAMEPRO_GET_CLOCK_COUNT(now);

		ConditionalParentScope* p_scope = m_ConditionalParentScopeList.GetHead();
		while (p_scope)
		{
			int64 max_duration = (p_scope->m_PreDuration * m_ClockFrequency) / 1000000;
			
			// throw away send buffers that are too old
			SendBuffer* p_send_buffer = p_scope->m_ChildSendBuffers.GetHead();
			while(p_send_buffer && now - p_send_buffer->GetCreationTime() > max_duration)
			{
				p_scope->m_ChildSendBuffers.RemoveHead();
				p_send_buffer->ClearSize();
				AddEmptySendBuffer(p_send_buffer);
				p_send_buffer = p_scope->m_ChildSendBuffers.GetHead();
			}

			p_scope = p_scope->GetNext();
		}
	}

	//------------------------------------------------------------------------
	// this is called when a normal scope ends. If the hires timers are within
	// this scope send them.
	void FrameProTLS::SendHiResTimersScope(int64 start_time, int64 end_time)
	{
		FRAMEPRO_ASSERT(IsOnTLSThread());

		FRAMEPRO_ASSERT(m_FirstHiresTimerTime < end_time);

		if(m_FirstHiresTimerTime < start_time)
			return;

		int count = m_HiResTimers.GetCount();
		FRAMEPRO_ASSERT(count);

		int size_to_send = sizeof(HiResTimerScopePacket) + count * sizeof(HiResTimerScopePacket::HiResTimer);

		// if we are connecting to FramePro, FramePro will ask for the string value later, otherwise we need to send it now
		if (m_SendStringsImmediately)
		{
			for(int i=0; i<count; ++i)
				SendString(m_HiResTimers[i].mp_Name, PacketType::StringLiteralTimerNamePacket);
		}

		{
			CriticalSectionScope lock(m_CurrentSendBufferCriticalSection);

			HiResTimerScopePacket* p_packet = (HiResTimerScopePacket*)AllocateSpaceInBuffer(size_to_send);
			p_packet->m_PacketType = PacketType::HiResTimerScopePacket;
			p_packet->m_StartTime = start_time;
			p_packet->m_EndTime = end_time;
			p_packet->m_Count = count;
			p_packet->m_ThreadId = m_ThreadId;
			p_packet->m_Padding = 0;

			HiResTimerScopePacket::HiResTimer* p_send_hires_timer = (HiResTimerScopePacket::HiResTimer*)(p_packet + 1);
			memcpy(p_send_hires_timer, &m_HiResTimers[0], count * sizeof(HiResTimer));
		}

		m_HiResTimers.ClearNoFree();

		m_FirstHiresTimerTime = 0;
	}

	//------------------------------------------------------------------------
	void FrameProTLS::SetCustomTimeSpanStat(StringId name, int64 value)
	{
		FRAMEPRO_ASSERT(IsOnTLSThread());

		int64 time;
		FRAMEPRO_GET_CLOCK_COUNT(time);

		{
			CriticalSectionScope lock(m_CurrentSendBufferCriticalSection);

			TimeSpanCustomStatPacket* p_packet = (TimeSpanCustomStatPacket*)AllocateSpaceInBuffer(sizeof(TimeSpanCustomStatPacket));
			p_packet->m_PacketType = PacketType::TimeSpanCustomStatPacket;
			p_packet->m_ThreadId = m_ThreadId;
			p_packet->m_ValueType = CustomStatValueType::Int64;
			p_packet->m_Name = name;
			p_packet->m_ValueInt64 = value;
			p_packet->m_ValueDouble = 0.0;
			p_packet->m_Time = time;
		}

		m_HiResTimers.ClearNoFree();
	}

	//------------------------------------------------------------------------
	void FrameProTLS::SetCustomTimeSpanStat(StringId name, double value)
	{
		FRAMEPRO_ASSERT(IsOnTLSThread());

		int64 time;
		FRAMEPRO_GET_CLOCK_COUNT(time);

		{
			CriticalSectionScope lock(m_CurrentSendBufferCriticalSection);

			TimeSpanCustomStatPacket* p_packet = (TimeSpanCustomStatPacket*)AllocateSpaceInBuffer(sizeof(TimeSpanCustomStatPacket));
			p_packet->m_PacketType = PacketType::TimeSpanCustomStatPacket;
			p_packet->m_ThreadId = m_ThreadId;
			p_packet->m_ValueType = CustomStatValueType::Double;
			p_packet->m_Name = name;
			p_packet->m_ValueInt64 = 0;
			p_packet->m_ValueDouble = value;
			p_packet->m_Time = time;
		}

		m_HiResTimers.ClearNoFree();
	}

	//------------------------------------------------------------------------
	void FrameProTLS::SetCustomTimeSpanStatW(StringId name, int64 value)
	{
		FRAMEPRO_ASSERT(IsOnTLSThread());

		int64 time;
		FRAMEPRO_GET_CLOCK_COUNT(time);

		{
			CriticalSectionScope lock(m_CurrentSendBufferCriticalSection);

			TimeSpanCustomStatPacket* p_packet = (TimeSpanCustomStatPacket*)AllocateSpaceInBuffer(sizeof(TimeSpanCustomStatPacket));
			p_packet->m_PacketType = PacketType::TimeSpanCustomStatPacketW;
			p_packet->m_ThreadId = m_ThreadId;
			p_packet->m_ValueType = CustomStatValueType::Int64;
			p_packet->m_Name = name;
			p_packet->m_ValueInt64 = value;
			p_packet->m_ValueDouble = 0.0;
			p_packet->m_Time = time;
		}

		m_HiResTimers.ClearNoFree();
	}

	//------------------------------------------------------------------------
	void FrameProTLS::SetCustomTimeSpanStatW(StringId name, double value)
	{
		FRAMEPRO_ASSERT(IsOnTLSThread());

		int64 time;
		FRAMEPRO_GET_CLOCK_COUNT(time);

		{
			CriticalSectionScope lock(m_CurrentSendBufferCriticalSection);

			TimeSpanCustomStatPacket* p_packet = (TimeSpanCustomStatPacket*)AllocateSpaceInBuffer(sizeof(TimeSpanCustomStatPacket));
			p_packet->m_PacketType = PacketType::TimeSpanCustomStatPacketW;
			p_packet->m_ThreadId = m_ThreadId;
			p_packet->m_ValueType = CustomStatValueType::Double;
			p_packet->m_Name = name;
			p_packet->m_ValueInt64 = 0;
			p_packet->m_ValueDouble = value;
			p_packet->m_Time = time;
		}

		m_HiResTimers.ClearNoFree();
	}

	//------------------------------------------------------------------------
	bool FrameProTLS::HaveSentCustomStatInfo(StringId name)
	{
		int name_index = (int)name;
		return name_index < m_InitialisedCustomStats.GetCount() && m_InitialisedCustomStats[name_index];
	}

	//------------------------------------------------------------------------
	void FrameProTLS::SetHaveSentCustomStatInfo(StringId name)
	{
		int name_index = (int)name;

		int old_count = m_InitialisedCustomStats.GetCount();
		if (old_count <= name_index)
		{
			int new_count = name_index + 1;
			
			if (!m_InitialisedCustomStats.Resize(new_count))
				return;

			memset(&m_InitialisedCustomStats[old_count], 0, (new_count - old_count) * sizeof(bool));
		}
		m_InitialisedCustomStats[name_index] = true;
	}

	//------------------------------------------------------------------------
	void FrameProTLS::SetCustomStatInfo(const char* p_name, const char* p_graph, const char* p_unit, uint colour)
	{
		StringId name = RegisterString(p_name);

		if (!HaveSentCustomStatInfo(name))
		{
			SendCustomStatGraphPacket(name, RegisterString(p_graph));
			SendCustomStatUnitPacket(name, RegisterString(p_unit));

			if(colour)
				SendCustomStatColourPacket(name, colour);

			SetHaveSentCustomStatInfo(name);
		}
	}
		
	//------------------------------------------------------------------------
	void FrameProTLS::SetCustomStatInfo(const wchar_t* p_name, const wchar_t* p_graph, const wchar_t* p_unit, uint colour)
	{
		StringId name = RegisterString(p_name);

		if (!HaveSentCustomStatInfo(name))
		{
			SendCustomStatGraphPacket(name, RegisterString(p_graph));
			SendCustomStatUnitPacket(name, RegisterString(p_unit));

			if(colour)
				SendCustomStatColourPacket(name, colour);

			SetHaveSentCustomStatInfo(name);
		}
	}

	//------------------------------------------------------------------------
	void FrameProTLS::SetCustomStatInfo(StringId name, const char* p_graph, const char* p_unit, uint colour)
	{
		if (!HaveSentCustomStatInfo(name))
		{
			SendCustomStatGraphPacket(name, RegisterString(p_graph));
			SendCustomStatUnitPacket(name, RegisterString(p_unit));

			if(colour)
				SendCustomStatColourPacket(name, colour);

			SetHaveSentCustomStatInfo(name);
		}
	}

	//------------------------------------------------------------------------
	void FrameProTLS::SetCustomStatInfo(StringId name, const wchar_t* p_graph, const wchar_t* p_unit, uint colour)
	{
		if (!HaveSentCustomStatInfo(name))
		{
			SendCustomStatGraphPacket(name, RegisterString(p_graph));
			SendCustomStatUnitPacket(name, RegisterString(p_unit));

			if(colour)
				SendCustomStatColourPacket(name, colour);

			SetHaveSentCustomStatInfo(name);
		}
	}

	//------------------------------------------------------------------------
	void FrameProTLS::SetCustomStatInfo(StringId name, StringId graph, StringId unit, uint colour)
	{
		if (!HaveSentCustomStatInfo(name))
		{
			SendCustomStatGraphPacket(name, graph);
			SendCustomStatUnitPacket(name, unit);

			if(colour)
				SendCustomStatColourPacket(name, colour);

			SetHaveSentCustomStatInfo(name);
		}
	}

	//------------------------------------------------------------------------
#if FRAMEPRO_ENABLE_CALLSTACKS
	CallstackResult FrameProTLS::GetCallstack()
	{
		return m_StackTrace.Capture();
	}
#endif

	//------------------------------------------------------------------------
	int FrameProTLS::GetAndClearCurrentCallstackId(int64 start_time, int64 end_time)
	{
		if (m_CurrentCallstackId == -1)
			return -1;

		int64 callstack_time = m_CurrentCallstackIdTime;

		if (callstack_time >= start_time && callstack_time < end_time)
		{
			int callstack_id = m_CurrentCallstackId;
			m_CurrentCallstackIdTime = 0;
			m_CurrentCallstackId = -1;
			return callstack_id;
		}

		return -1;
	}

	//------------------------------------------------------------------------
	void FrameProTLS::SetCurrentCallstackId(int id)
	{
		m_CurrentCallstackId = id;
		FRAMEPRO_GET_CLOCK_COUNT(m_CurrentCallstackIdTime);
	}
}

//------------------------------------------------------------------------
#if FRAMEPRO_WIN_BASED_PLATFORM
	#pragma warning(pop)
#endif


//------------------------------------------------------------------------
//
// IncrementingBlockAllocator.cpp
//

//------------------------------------------------------------------------
namespace FramePro
{
	//------------------------------------------------------------------------
	IncrementingBlockAllocator::IncrementingBlockAllocator(Allocator* p_allocator)
	:	mp_Allocator(p_allocator),
		mp_BlockList(NULL),
		m_CurrentBlockSize(m_MemoryBlockSize),
		m_MemorySize(0)
	{
	}

	//------------------------------------------------------------------------
	IncrementingBlockAllocator::~IncrementingBlockAllocator()
	{
		Clear();
	}

	//------------------------------------------------------------------------
	void IncrementingBlockAllocator::Clear()
	{
		Block* p_block = mp_BlockList;
		while(p_block)
		{
			Block* p_next = p_block->mp_Next;
			mp_Allocator->Free(p_block);
			p_block = p_next;
		}

		mp_BlockList = NULL;
		m_CurrentBlockSize = m_MemoryBlockSize;
		m_MemorySize = 0;
	}

	//------------------------------------------------------------------------
	void* IncrementingBlockAllocator::Alloc(size_t size)
	{
		if (m_CurrentBlockSize + size > m_MemoryBlockSize)
		{
			if (!AllocateBlock())
				return NULL;
		}

		void* p_mem = mp_BlockList->m_Memory + m_CurrentBlockSize;
		m_CurrentBlockSize += size;
		return p_mem;
	}

	//------------------------------------------------------------------------
	bool IncrementingBlockAllocator::AllocateBlock()
	{
		Block* p_block = (Block*)mp_Allocator->Alloc(sizeof(Block));
		if (!p_block)
			return false;

		p_block->mp_Next = mp_BlockList;
		mp_BlockList = p_block;
		m_CurrentBlockSize = 0;

		m_MemorySize += m_BlockSize;

		return true;
	}
}


//------------------------------------------------------------------------
//
// PointerSet.cpp
//

//------------------------------------------------------------------------
namespace FramePro
{
	//------------------------------------------------------------------------
	const int g_InitialCapacity = 32;

	//------------------------------------------------------------------------
	PointerSet::PointerSet(Allocator* p_allocator)
	:	mp_Data((const void**)p_allocator->Alloc(g_InitialCapacity*sizeof(const void*))),
		m_CapacityMask(g_InitialCapacity-1),
		m_Count(0),
		m_Capacity(g_InitialCapacity),
		mp_Allocator(p_allocator)
	{
		if(mp_Data)
			memset(mp_Data, 0, g_InitialCapacity*sizeof(const void*));
	}

	//------------------------------------------------------------------------
	PointerSet::~PointerSet()
	{
		mp_Allocator->Free(mp_Data);
	}

	//------------------------------------------------------------------------
	bool PointerSet::Grow()
	{
		int old_capacity = m_Capacity;
		const void** p_old_data = mp_Data;

		// allocate a new set
		m_Capacity = m_Capacity ? 2*m_Capacity : 32;
		FRAMEPRO_ASSERT(m_Capacity < (int)(INT_MAX/sizeof(void*)));

		m_CapacityMask = m_Capacity - 1;
		size_t alloc_size = m_Capacity * sizeof(const void*);
		mp_Data = (const void**)mp_Allocator->Alloc(alloc_size);
		if (!mp_Data)
			return false;

		int size = m_Capacity * sizeof(void*);
		memset(mp_Data, 0, size);

		// transfer pointers from old set
		m_Count = 0;
		for(int i=0; i<old_capacity; ++i)
		{
			const void* p = p_old_data[i];
			if(p)
				Add(p);
		}

		// release old buffer
		mp_Allocator->Free(p_old_data);

		alloc_size -= old_capacity * sizeof(const void*);

		return true;
	}

	//------------------------------------------------------------------------
	// return true if added, false if already in set
	bool PointerSet::AddInternal(const void* p, int64 hash, int index)
	{
		if(m_Count >= m_Capacity/4)
		{
			if (!Grow())
				return false;

			index = hash & m_CapacityMask;
		}

		const void* p_existing = mp_Data[index];
		while(p_existing)
		{
			if(p_existing == p)
				return false;
			index = (index + 1) & m_CapacityMask;
			p_existing = mp_Data[index];
		}

		mp_Data[index] = p;

		++m_Count;

		return true;
	}
}


//------------------------------------------------------------------------
//
// SendBuffer.cpp
//

//------------------------------------------------------------------------
namespace FramePro
{
	//------------------------------------------------------------------------
	SendBuffer::SendBuffer(Allocator* p_allocator, int capacity, FrameProTLS* p_owner)
	:	mp_Buffer(NULL),
		m_Size(0),
		m_Capacity(0),
		mp_Next(NULL),
		mp_Allocator(p_allocator),
		mp_Owner(p_owner),
		m_CreationTime(0)
	{
		SetCreationTime();

		AllocateBuffer(capacity);
	}

	//------------------------------------------------------------------------
	SendBuffer::~SendBuffer()
	{
		ClearBuffer();
	}

	//------------------------------------------------------------------------
	bool SendBuffer::AllocateBuffer(int capacity)
	{
		FRAMEPRO_ASSERT(!mp_Buffer);

		mp_Buffer = mp_Allocator->Alloc(capacity);
		if (!mp_Buffer)
			return false;

		m_Capacity = capacity;

		return true;
	}

	//------------------------------------------------------------------------
	void SendBuffer::ClearBuffer()
	{
		if(mp_Buffer)
		{
			mp_Allocator->Free(mp_Buffer);
			mp_Buffer = NULL;
		}

		m_Size = 0;
		m_Capacity = 0;
	}

	//------------------------------------------------------------------------
	void SendBuffer::SetCreationTime()
	{
		FRAMEPRO_GET_CLOCK_COUNT(m_CreationTime);
	}
}


//------------------------------------------------------------------------
//
// Socket.cpp
//

//------------------------------------------------------------------------
#if FRAMEPRO_SOCKETS_ENABLED

//------------------------------------------------------------------------
namespace FramePro
{
	//------------------------------------------------------------------------
	Socket::Socket()
	:	m_Listening(false)
	{
		Platform::CreateSocket(m_OSSocketMem, sizeof(m_OSSocketMem));
	}

	//------------------------------------------------------------------------
	Socket::~Socket()
	{
		Platform::UninitialiseSocketSystem();
	}

	//------------------------------------------------------------------------
	void Socket::Disconnect()
	{
		Platform::DisconnectSocket(m_OSSocketMem, m_Listening);
	}

	//------------------------------------------------------------------------
	bool Socket::StartListening()
	{
		FRAMEPRO_ASSERT(IsValid());

		if(!Platform::StartSocketListening(m_OSSocketMem))
		{
			Platform::HandleSocketError();
			return false;
		}

		m_Listening = true;

		return true;
	}

	//------------------------------------------------------------------------
	bool Socket::Bind(const char* p_port)
	{
		FRAMEPRO_ASSERT(!IsValid());

		if(!Platform::InitialiseSocketSystem())
			return false;

		return Platform::BindSocket(m_OSSocketMem, p_port);
	}

	//------------------------------------------------------------------------
	bool Socket::Accept(Socket& client_socket)
	{
		FRAMEPRO_ASSERT(!client_socket.IsValid());
		return Platform::AcceptSocket(m_OSSocketMem, client_socket.m_OSSocketMem);
	}

	//------------------------------------------------------------------------
	bool Socket::Send(const void* p_buffer, size_t size)
	{
		FRAMEPRO_ASSERT(size >= 0 && size <= INT_MAX);

		int bytes_to_send = (int)size;
		while(bytes_to_send != 0)
		{
			int bytes_sent = 0;
			if(!Platform::SocketSend(m_OSSocketMem, p_buffer, bytes_to_send, bytes_sent))
			{
				Platform::HandleSocketError();
				Disconnect();
				return false;
			}
			p_buffer = (char*)p_buffer + bytes_sent;
			bytes_to_send -= bytes_sent;
		}

		return true;
	}

	//------------------------------------------------------------------------
	int Socket::Receive(void* p_buffer, int size)
	{
		int total_bytes_received = 0;

		while(size)
		{
			int bytes_received = 0;
			bool result = Platform::SocketReceive(m_OSSocketMem, (char*)p_buffer, size, bytes_received);

			if (!result)
			{
				Platform::HandleSocketError();
				Disconnect();
				return total_bytes_received;
			}
			else if(bytes_received == 0)
			{
				Disconnect();
				return bytes_received;
			}

			total_bytes_received += bytes_received;

			size -= bytes_received;
			FRAMEPRO_ASSERT(size >= 0);

			p_buffer = (char*)p_buffer + bytes_received;
		}

		return total_bytes_received;
	}

	//------------------------------------------------------------------------
	bool Socket::IsValid() const
	{
		return Platform::IsSocketValid(m_OSSocketMem);
	}
}

//------------------------------------------------------------------------
#endif		// #if FRAMEPRO_SOCKETS_ENABLED


//------------------------------------------------------------------------
//
// Thread.cpp
//

//------------------------------------------------------------------------
namespace FramePro
{
	//------------------------------------------------------------------------
	Thread::Thread()
	:	m_Created(false),
		m_Alive(false),
		m_ThreadTerminatedEvent(false, false)
	{
	}

	//------------------------------------------------------------------------
	Thread::~Thread()
	{
		if(m_Created)
			Platform::DestroyThread(m_OSThread);
	}

	//------------------------------------------------------------------------
	bool Thread::CreateThread(FramePro::ThreadMain p_thread_main, void* p_param, Allocator* p_allocator)
	{
		if(m_Created)
			Platform::DestroyThread(m_OSThread);

		mp_ThreadMain = p_thread_main;
		mp_Param = p_param;

		if (Platform::CreateThread(m_OSThread, sizeof(m_OSThread), ThreadMain, this, p_allocator))
			m_Created = true;

		return m_Created;
	}

	//------------------------------------------------------------------------
	int Thread::ThreadMain(void* p_context)
	{
		Thread* p_thread = (Thread*)p_context;
		p_thread->m_Alive = true;
		unsigned long ret = (unsigned long)p_thread->mp_ThreadMain(p_thread->mp_Param);
		p_thread->m_Alive = false;
		p_thread->m_ThreadTerminatedEvent.Set();
		return ret;
	}

	//------------------------------------------------------------------------
	void Thread::SetPriority(int priority)
	{
		Platform::SetThreadPriority(m_OSThread, priority);
	}

	//------------------------------------------------------------------------
	void Thread::SetAffinity(int affinity)
	{
		Platform::SetThreadAffinity(m_OSThread, affinity);
	}

	//------------------------------------------------------------------------
	void Thread::WaitForThreadToTerminate(int timeout)
	{
		if (m_Created)
			m_ThreadTerminatedEvent.Wait(timeout);
	}
}


//------------------------------------------------------------------------
//
// EnumModulesLinux.h
//

//------------------------------------------------------------------------
#if FRAMEPRO_LINUX_BASED_PLATFORM && !FRAMEPRO_PLATFORM_PS4

//------------------------------------------------------------------------
#if FRAMEPRO_ENABLE_CALLSTACKS

//------------------------------------------------------------------------
namespace FramePro
{
	//------------------------------------------------------------------------
	namespace EnumModulesLinux
	{
		void EnumerateModules(Array<ModulePacket*>& module_packets, Allocator* p_allocator);
	}
}

//------------------------------------------------------------------------
#endif		// #if FRAMEPRO_ENABLE_CALLSTACKS

//------------------------------------------------------------------------
#endif		// #if FRAMEPRO_LINUX_BASED_PLATFORM


//------------------------------------------------------------------------
//
// EnumModulesLinux.cpp
//

//------------------------------------------------------------------------
#if FRAMEPRO_LINUX_BASED_PLATFORM && !FRAMEPRO_PLATFORM_PS4

//------------------------------------------------------------------------
#if FRAMEPRO_ENABLE_CALLSTACKS

//------------------------------------------------------------------------
#if FRAMEPRO_ENUMERATE_ALL_MODULES
	#include <link.h>
#endif

//------------------------------------------------------------------------
#include <sys/types.h>
#include <unistd.h>

//------------------------------------------------------------------------
namespace FramePro
{
	//------------------------------------------------------------------------
	void BaseAddressLookupFunction()
	{
	}

	//------------------------------------------------------------------------
#if FRAMEPRO_ENUMERATE_ALL_MODULES

	//------------------------------------------------------------------------
	struct ModuleCallbackContext
	{
		Array<ModulePacket*>* mp_ModulePackets;
		Allocator* mp_Allocator;
	};

	//------------------------------------------------------------------------
	void EnumerateLoadedModulesCallback(
		int64 module_base,
		const char* p_module_name,
		bool use_lookup_function_for_base_address,
		ModuleCallbackContext* p_context)
	{
		ModulePacket* p_module_packet = (ModulePacket*)p_context->mp_Allocator->Alloc(sizeof(ModulePacket));
		if (!p_module_packet)
			return;

		memset(p_module_packet, 0, sizeof(ModulePacket));
		p_module_packet->m_PacketType = PacketType::ModulePacket;

		p_module_packet->m_ModuleBase = module_base;

		size_t module_name_length = strlen(p_module_name) + 1;
		FRAMEPRO_ASSERT(sizeof(p_module_packet->m_ModuleName) >= module_name_length);
		memcpy(p_module_packet->m_ModuleName, p_module_name, module_name_length);

		const char* p_last_slash = strrchr(p_module_name, '/');
		p_last_slash = p_last_slash ? p_last_slash + 1 : p_module_name;
		char filename[FRAMEPRO_MAX_PATH];
		sprintf(filename, "%s.sym_txt", p_last_slash);
		memcpy(p_module_packet->m_SymbolFilename, filename, strlen(filename) + 1);
		
		p_module_packet->m_UseLookupFunctionForBaseAddress = use_lookup_function_for_base_address ? 1 : 0;

		p_context->mp_ModulePackets->Add(p_module_packet);
	}

	//------------------------------------------------------------------------
	int EnumerateLoadedModulesCallback(struct dl_phdr_info* info, size_t size, void* data)
	{
		ModuleCallbackContext* p_context = (ModuleCallbackContext*)data;

		int64 module_base = 0;
		for (int j = 0; j < info->dlpi_phnum; j++)
		{
			if (info->dlpi_phdr[j].p_type == PT_LOAD)
			{
				module_base = info->dlpi_addr + info->dlpi_phdr[j].p_vaddr;
				break;
			}
		}

		static bool first = true;
		if (first)
		{
			first = false;

			module_base = (int64)BaseAddressLookupFunction;		// use the address of the BaseAddressLookupFunction function so that we can work it out later

			// get the module name
			char arg1[20];
			char char_filename[FRAMEPRO_MAX_PATH];
			sprintf(arg1, "/proc/%d/exe", getpid());
			memset(char_filename, 0, FRAMEPRO_MAX_PATH);
			readlink(arg1, char_filename, FRAMEPRO_MAX_PATH - 1);

			EnumerateLoadedModulesCallback(
				module_base,
				char_filename,
				true,
				p_context);
		}
		else
		{
			EnumerateLoadedModulesCallback(
				module_base,
				info->dlpi_name,
				false,
				p_context);
		}

		return 0;
	}
#endif			// #if FRAMEPRO_ENUMERATE_ALL_MODULES

	//------------------------------------------------------------------------
	void EnumModulesLinux::EnumerateModules(Array<ModulePacket*>& module_packets, Allocator* p_allocator)
	{
		// if you are having problems compiling this on your platform unset FRAMEPRO_ENUMERATE_ALL_MODULES and it send info for just the main module
#if FRAMEPRO_ENUMERATE_ALL_MODULES
		ModuleCallbackContext* p_context = (ModuleCallbackContext*)p_allocator->Alloc(sizeof(ModuleCallbackContext));
		if (!p_context)
			return;

		p_context->mp_ModulePackets = &module_packets;
		p_context->mp_Allocator = p_allocator;

		dl_iterate_phdr(EnumerateLoadedModulesCallback, p_context);

		Delete(p_allocator, p_context);
#endif

		if (!module_packets.GetCount())
		{
			// if FRAMEPRO_ENUMERATE_ALL_MODULES is set or enumeration failed for some reason, fall back
			// to getting the base address for the main module. This will always work for for all platforms.

			ModulePacket* p_module_packet = (ModulePacket*)p_allocator->Alloc(sizeof(ModulePacket));
			if (!p_module_packet)
				return;

			memset(p_module_packet, 0, sizeof(ModulePacket));

			p_module_packet->m_PacketType = PacketType::ModulePacket;
			p_module_packet->m_UseLookupFunctionForBaseAddress = 0;

			p_module_packet->m_UseLookupFunctionForBaseAddress = 1;

			p_module_packet->m_ModuleBase = (int64)BaseAddressLookupFunction;		// use the address of the BaseAddressLookupFunction function so that we can work it out later

			// get the module name
			char arg1[20];
			sprintf(arg1, "/proc/%d/exe", getpid());
			memset(p_module_packet->m_ModuleName, 0, FRAMEPRO_MAX_PATH);
			readlink(arg1, p_module_packet->m_ModuleName, FRAMEPRO_MAX_PATH - 1);

			const char* p_last_slash = strrchr(p_module_packet->m_ModuleName, '/');
			p_last_slash = p_last_slash ? p_last_slash + 1 : p_module_packet->m_ModuleName;
			char filename[FRAMEPRO_MAX_PATH];
			sprintf(filename, "%s.sym_txt", p_last_slash);
			memcpy(p_module_packet->m_SymbolFilename, filename, strlen(filename) + 1);
		
			module_packets.Add(p_module_packet);
		}
	}
}

//------------------------------------------------------------------------
#endif		// #if FRAMEPRO_ENABLE_CALLSTACKS

//------------------------------------------------------------------------
#endif		// #if FRAMEPRO_LINUX_BASED_PLATFORM


//------------------------------------------------------------------------
//
// EnumModulesWindows.h
//

//------------------------------------------------------------------------
#if FRAMEPRO_WIN_BASED_PLATFORM

//------------------------------------------------------------------------
#if FRAMEPRO_ENABLE_CALLSTACKS

//------------------------------------------------------------------------
namespace FramePro
{
	//------------------------------------------------------------------------
	namespace EnumModulesWindows
	{
		void EnumerateModules(Array<ModulePacket*>& module_packets, Allocator* p_allocator);
	}
}

//------------------------------------------------------------------------
#endif		// #if FRAMEPRO_ENABLE_CALLSTACKS

//------------------------------------------------------------------------
#endif		// #if FRAMEPRO_WIN_BASED_PLATFORM


//------------------------------------------------------------------------
//
// EnumModulesWindows.cpp
//

//------------------------------------------------------------------------
#if FRAMEPRO_WIN_BASED_PLATFORM

//------------------------------------------------------------------------
#if FRAMEPRO_ENABLE_CALLSTACKS

//------------------------------------------------------------------------
#if FRAMEPRO_PLATFORM_UE4
	#include "Windows/AllowWindowsPlatformTypes.h"
#endif
#pragma warning(push)
#pragma warning(disable:4668)
#pragma warning(disable:4091)
	#ifndef WIN32_LEAN_AND_MEAN
		#define WIN32_LEAN_AND_MEAN
	#endif
	#include <windows.h>
	#if FRAMEPRO_ENUMERATE_ALL_MODULES
		#include <Dbghelp.h>
	#endif
#pragma warning(pop)
#if FRAMEPRO_PLATFORM_UE4
	#include "Windows/HideWindowsPlatformTypes.h"
#endif

#if FRAMEPRO_ENUMERATE_ALL_MODULES
	#pragma comment(lib, "Dbghelp.lib")
#endif

//------------------------------------------------------------------------
namespace FramePro
{
	#if FRAMEPRO_ENUMERATE_ALL_MODULES
		//------------------------------------------------------------------------
		struct CV_HEADER
		{
			int Signature;
			int Offset;
		};

		struct CV_INFO_PDB20
		{
			CV_HEADER CvHeader;
			int Signature;
			int Age;
			char PdbFileName[FRAMEPRO_MAX_PATH];
		};

		struct CV_INFO_PDB70
		{
			int  CvSignature;
			GUID Signature;
			int Age;
			char PdbFileName[FRAMEPRO_MAX_PATH];
		};

		//------------------------------------------------------------------------
		struct ModuleCallbackContext
		{
			Array<ModulePacket*>* mp_ModulePackets;
			Allocator* mp_Allocator;
		};

		//------------------------------------------------------------------------
		void GetExtraModuleInfo(int64 ModuleBase, ModulePacket* p_module_packet)
		{
			IMAGE_DOS_HEADER* p_dos_header = (IMAGE_DOS_HEADER*)ModuleBase;
			IMAGE_NT_HEADERS* p_nt_header = (IMAGE_NT_HEADERS*)((char*)ModuleBase + p_dos_header->e_lfanew);
			IMAGE_OPTIONAL_HEADER& optional_header = p_nt_header->OptionalHeader;
			IMAGE_DATA_DIRECTORY& image_data_directory = optional_header.DataDirectory[IMAGE_DIRECTORY_ENTRY_DEBUG];
			IMAGE_DEBUG_DIRECTORY* p_debug_info_array = (IMAGE_DEBUG_DIRECTORY*)(ModuleBase + image_data_directory.VirtualAddress);
			int count = image_data_directory.Size / sizeof(IMAGE_DEBUG_DIRECTORY);
			for(int i=0; i<count; ++i)
			{
				if(p_debug_info_array[i].Type == IMAGE_DEBUG_TYPE_CODEVIEW)
				{
					char* p_cv_data = (char*)(ModuleBase + p_debug_info_array[i].AddressOfRawData);
					if(strncmp(p_cv_data, "RSDS", 4) == 0)
					{
						CV_INFO_PDB70* p_cv_info = (CV_INFO_PDB70*)p_cv_data;

						p_module_packet->m_PacketType = PacketType::ModulePacket;
						p_module_packet->m_Age = p_cv_info->Age;
					
						static_assert(sizeof(p_module_packet->m_Sig) == sizeof(p_cv_info->Signature), "sig size wrong");
						memcpy(p_module_packet->m_Sig, &p_cv_info->Signature, sizeof(p_cv_info->Signature));

						FRAMEPRO_ASSERT(strlen(p_cv_info->PdbFileName) < sizeof(p_module_packet->m_SymbolFilename));
						memcpy(p_module_packet->m_SymbolFilename, p_cv_info->PdbFileName, strlen(p_cv_info->PdbFileName) + 1);

						return;									// returning here
					}
					else if(strncmp(p_cv_data, "NB10", 4) == 0)
					{
						CV_INFO_PDB20* p_cv_info = (CV_INFO_PDB20*)p_cv_data;

						p_module_packet->m_PacketType = PacketType::ModulePacket;
						p_module_packet->m_Age = p_cv_info->Age;

						memset(p_module_packet->m_Sig, 0, sizeof(p_module_packet->m_Sig));
						static_assert(sizeof(p_cv_info->Signature) <= sizeof(p_module_packet->m_Sig), "sig size wrong");
						memcpy(p_module_packet->m_Sig, &p_cv_info->Signature, sizeof(p_cv_info->Signature));

						FRAMEPRO_ASSERT(sizeof(p_module_packet->m_SymbolFilename) >= strlen(p_cv_info->PdbFileName) + 1);
						memcpy(p_module_packet->m_SymbolFilename, p_cv_info->PdbFileName, strlen(p_cv_info->PdbFileName) + 1);

						return;									// returning here
					}
				}
			}
		}

		//------------------------------------------------------------------------
		void EnumerateLoadedModulesCallback(
			int64 module_base,
			const char* p_module_name,
			bool use_lookup_function_for_base_address,
			ModuleCallbackContext* p_context)
		{
			ModulePacket* p_module_packet = (ModulePacket*)p_context->mp_Allocator->Alloc(sizeof(ModulePacket));
			if (!p_module_packet)
				return;

			memset(p_module_packet, 0, sizeof(ModulePacket));
			p_module_packet->m_PacketType = PacketType::ModulePacket;

			p_module_packet->m_ModuleBase = module_base;
			
			size_t module_name_length = strlen(p_module_name) + 1;
			FRAMEPRO_ASSERT(sizeof(p_module_packet->m_ModuleName) >= module_name_length);
			memcpy(p_module_packet->m_ModuleName, p_module_name, module_name_length);

			p_module_packet->m_UseLookupFunctionForBaseAddress = use_lookup_function_for_base_address ? 1 : 0;

			GetExtraModuleInfo(module_base, p_module_packet);

			if (!p_context->mp_ModulePackets->Add(p_module_packet))
				return;
		}

		#if !defined(_IMAGEHLP_SOURCE_) && defined(_IMAGEHLP64)
		// depending on your platform you may need to change PCSTR to PSTR for ModuleName
		BOOL CALLBACK EnumerateLoadedModulesCallback(__in PCSTR ModuleName,__in DWORD64 ModuleBase,__in ULONG,__in_opt PVOID UserContext)
		#else
		BOOL CALLBACK EnumerateLoadedModulesCallback(__in PCSTR ModuleName,__in ULONG ModuleBase,__in ULONG,__in_opt PVOID UserContext)
		#endif
		{
			ModuleCallbackContext* p_context = (ModuleCallbackContext*)UserContext;

			EnumerateLoadedModulesCallback(ModuleBase, ModuleName, false, p_context);

			return true;
		}
	#endif			// #if FRAMEPRO_ENUMERATE_ALL_MODULES

	//------------------------------------------------------------------------
	void EnumModulesWindows::EnumerateModules(Array<ModulePacket*>& module_packets, Allocator* p_allocator)
	{
		// if you are having problems compiling this on your platform unset FRAMEPRO_ENUMERATE_ALL_MODULES and it send info for just the main module
		#if FRAMEPRO_ENUMERATE_ALL_MODULES
			ModuleCallbackContext* p_context = (ModuleCallbackContext*)p_allocator->Alloc(sizeof(ModuleCallbackContext));
			if (!p_context)
				return;

			p_context->mp_ModulePackets = &module_packets;
			p_context->mp_Allocator = p_allocator;

			EnumerateLoadedModules64(GetCurrentProcess(), EnumerateLoadedModulesCallback, p_context);

			Delete(p_allocator, p_context);
		#endif

		if (!module_packets.GetCount())
		{
			// if FRAMEPRO_ENUMERATE_ALL_MODULES is set or enumeration failed for some reason, fall back
			// to getting the base address for the main module. This will always work for for all platforms.

			ModulePacket* p_module_packet = (ModulePacket*)p_allocator->Alloc(sizeof(ModulePacket));
			if (!p_module_packet)
				return;

			memset(p_module_packet, 0, sizeof(ModulePacket));

			p_module_packet->m_PacketType = PacketType::ModulePacket;
			p_module_packet->m_UseLookupFunctionForBaseAddress = 0;

			static int module = 0;
			HMODULE module_handle = 0;
			GetModuleHandleEx(GET_MODULE_HANDLE_EX_FLAG_FROM_ADDRESS, (LPCTSTR)&module, &module_handle);

			p_module_packet->m_ModuleBase = (int64)module_handle;

			FRAMEPRO_TCHAR tchar_filename[FRAMEPRO_MAX_PATH] = { 0 };
			GetModuleFileName(NULL, tchar_filename, FRAMEPRO_MAX_PATH);

			#ifdef UNICODE
				size_t chars_converted = 0;
				wcstombs_s(&chars_converted, p_module_packet->m_ModuleName, tchar_filename, FRAMEPRO_MAX_PATH);
			#else
				strcpy_s(p_module_packet->m_ModuleName, tchar_filename);
			#endif

			if(!module_packets.Add(p_module_packet))
				return;
		}
	}
}

//------------------------------------------------------------------------
#endif		// #if FRAMEPRO_ENABLE_CALLSTACKS

//------------------------------------------------------------------------
#endif		// #if FRAMEPRO_WIN_BASED_PLATFORM


//------------------------------------------------------------------------
//
// FrameProPlatform.cpp
//
//------------------------------------------------------------------------
// ---
// --- FRAMEPRO PLATFORM IMPLEMENTATION START ---
// ---
//------------------------------------------------------------------------


//------------------------------------------------------------------------
#include <ctime>

//------------------------------------------------------------------------
// general Win/Linux implementations that apply to all platforms
//------------------------------------------------------------------------
#if FRAMEPRO_WIN_BASED_PLATFORM

	//------------------------------------------------------------------------
	#if FRAMEPRO_PLATFORM_WIN
		#pragma warning(push)
		#pragma warning(disable:4668)
		#if FRAMEPRO_PLATFORM_UE4
			#include "Windows/AllowWindowsPlatformTypes.h"
		#endif
		#ifndef WIN32_LEAN_AND_MEAN
			#define WIN32_LEAN_AND_MEAN
		#endif
		#include <windows.h>
		#include <psapi.h>
		#if FRAMEPRO_PLATFORM_UE4
			#include "Windows/HideWindowsPlatformTypes.h"
		#endif
		#pragma warning(pop)
	#endif

	//------------------------------------------------------------------------
	#if defined(_MSC_VER) && _MSC_VER <= 1600
		#error FramePro only supports Visual Studio 2012 and above. This is because it needs atomics. If you really need 2010 support please contact slynch@puredevsoftware.com
	#endif

	//------------------------------------------------------------------------
//@EPIC BEGIN: MAX_PATH might not be defined
	//#if FRAMEPRO_MAX_PATH != MAX_PATH
	//	#error
	//#endif
//@EPIC END

	//------------------------------------------------------------------------
	#if FRAMEPRO_SOCKETS_ENABLED
		#pragma comment(lib, "Ws2_32.lib")

		#if defined(AF_IPX) && !defined(_WINSOCK2API_)
			#error winsock already defined. Please include winsock2.h before including windows.h or use WIN32_LEAN_AND_MEAN. See the FAQ for more info.
		#endif
		#if FRAMEPRO_PLATFORM_UE4
			#include "Windows/AllowWindowsPlatformTypes.h"
		#endif
		#include <winsock2.h>
		#include <ws2tcpip.h>
		#if FRAMEPRO_PLATFORM_UE4
			#include "Windows/HideWindowsPlatformTypes.h"
		#endif
	#endif

	//------------------------------------------------------------------------
	namespace FramePro
	{
		//------------------------------------------------------------------------
		namespace GenericPlatform
		{
			//------------------------------------------------------------------------
			void DebugWrite(const char* p_string)
			{
				OutputDebugStringA(p_string);
			}

			//------------------------------------------------------------------------
			SRWLOCK& GetOSLock(void* p_os_lock_mem)
			{
				return *(SRWLOCK*)p_os_lock_mem;
			}

			//------------------------------------------------------------------------
			void CreateLock(void* p_os_lock_mem, int os_lock_mem_size)
			{
				FRAMEPRO_ASSERT(os_lock_mem_size >= sizeof(SRWLOCK));
				FRAMEPRO_UNREFERENCED(os_lock_mem_size);
				InitializeSRWLock(&GetOSLock(p_os_lock_mem));
			}

			//------------------------------------------------------------------------
			void DestroyLock(void*)
			{
				// do nothing
			}

			//------------------------------------------------------------------------
			void TakeLock(void* p_os_lock_mem)
			{
				AcquireSRWLockExclusive(&GetOSLock(p_os_lock_mem));
			}

			//------------------------------------------------------------------------
			void ReleaseLock(void* p_os_lock_mem)
			{
				ReleaseSRWLockExclusive(&GetOSLock(p_os_lock_mem));
			}

			//------------------------------------------------------------------------
			void GetLocalTime(tm* p_tm, const time_t *p_time)
			{
				localtime_s(p_tm, p_time);
			}

			//------------------------------------------------------------------------
			int GetCurrentProcessId()
			{
				return ::GetCurrentProcessId();
			}

			//------------------------------------------------------------------------
			void VSPrintf(char* p_buffer, size_t const buffer_size, const char* p_format, va_list arg_list)
			{
				vsprintf_s(p_buffer, buffer_size, p_format, arg_list);
			}

			//------------------------------------------------------------------------
			void ToString(int value, char* p_dest, int dest_size)
			{
				_itoa_s(value, p_dest, dest_size, 10);
			}

			//------------------------------------------------------------------------
			HANDLE& GetOSEventHandle(void* p_os_event_mem)
			{
				return *(HANDLE*)p_os_event_mem;
			}

			//------------------------------------------------------------------------
			void CreateEventX(void* p_os_event_mem, int os_event_mem_size, bool initial_state, bool auto_reset)
			{
				FRAMEPRO_ASSERT(os_event_mem_size >= sizeof(HANDLE));
				FRAMEPRO_UNREFERENCED(os_event_mem_size);
				GetOSEventHandle(p_os_event_mem) = ::CreateEvent(NULL, !auto_reset, initial_state, NULL);

			}

			//------------------------------------------------------------------------
			void DestroyEvent(void* p_os_event_mem)
			{
				CloseHandle(GetOSEventHandle(p_os_event_mem));
			}

			//------------------------------------------------------------------------
			void SetEvent(void* p_os_event_mem)
			{
				::SetEvent(GetOSEventHandle(p_os_event_mem));
			}

			//------------------------------------------------------------------------
			void ResetEvent(void* p_os_event_mem)
			{
				::ResetEvent(GetOSEventHandle(p_os_event_mem));
			}

			//------------------------------------------------------------------------
			int WaitEvent(void* p_os_event_mem, int timeout)
			{
				return WaitForSingleObject(GetOSEventHandle(p_os_event_mem), timeout) == 0/*WAIT_OBJECT_0*/;
			}

			//------------------------------------------------------------------------
			#if FRAMEPRO_SOCKETS_ENABLED
				volatile int g_WinSockInitialiseCount = 0;
			#endif

			//------------------------------------------------------------------------
			void HandleSocketError()
			{
				#if FRAMEPRO_SOCKETS_ENABLED && !FRAMEPRO_PLATFORM_XBOXONE
					if (WSAGetLastError() == WSAEADDRINUSE)
					{
						Platform::DebugWrite("FramePro: Network connection conflict. Please make sure that other FramePro enabled applications are shut down, or change the port in the the FramePro lib and FramePro settings.\n");
						return;
					}

					int buffer_size = 1024;
					FRAMEPRO_TCHAR* p_buffer = (FRAMEPRO_TCHAR*)HeapAlloc(GetProcessHeap(), 0, buffer_size * sizeof(FRAMEPRO_TCHAR));
					if (p_buffer)
					{
						memset(p_buffer, 0, buffer_size * sizeof(FRAMEPRO_TCHAR));

						FormatMessage(
							FORMAT_MESSAGE_FROM_SYSTEM,
							NULL,
							WSAGetLastError(),
							0,
							p_buffer,
							buffer_size,
							NULL);

						FramePro::DebugWrite("FramePro Network Error: %ws\n", p_buffer);

						HeapFree(GetProcessHeap(), 0, p_buffer);
					}
				#endif
			}

			//------------------------------------------------------------------------
			bool InitialiseSocketSystem()
			{
				#if FRAMEPRO_SOCKETS_ENABLED
					if (g_WinSockInitialiseCount == 0)
					{
						// Initialize Winsock
						WSADATA wsaData;
						if (WSAStartup(MAKEWORD(2, 2), &wsaData) != 0)
						{
							HandleSocketError();
							return false;
						}
					}

					++g_WinSockInitialiseCount;

					return true;
				#else
					return false;
				#endif
			}

			//------------------------------------------------------------------------
			void UninitialiseSocketSystem()
			{
				#if FRAMEPRO_SOCKETS_ENABLED
					--g_WinSockInitialiseCount;

					if (g_WinSockInitialiseCount == 0)
					{
						if (WSACleanup() == SOCKET_ERROR)
							HandleSocketError();
					}
				#endif
			}

			//------------------------------------------------------------------------
			#if FRAMEPRO_SOCKETS_ENABLED
				SOCKET& GetOSSocket(void* p_os_socket_mem)
				{
					return *(SOCKET*)p_os_socket_mem;
				}
			#endif

			//------------------------------------------------------------------------
			#if FRAMEPRO_SOCKETS_ENABLED
				const SOCKET& GetOSSocket(const void* p_os_socket_mem)
				{
					return *(const SOCKET*)p_os_socket_mem;
				}
			#endif

			//------------------------------------------------------------------------
			void CreateSocket(void* p_os_socket_mem, int os_socket_mem_size)
			{
				#if FRAMEPRO_SOCKETS_ENABLED
					FRAMEPRO_ASSERT(os_socket_mem_size >= sizeof(SOCKET));
					FRAMEPRO_UNREFERENCED(os_socket_mem_size);
					new (p_os_socket_mem)SOCKET();
					GetOSSocket(p_os_socket_mem) = INVALID_SOCKET;
				#else
					FRAMEPRO_UNREFERENCED(p_os_socket_mem);
					FRAMEPRO_UNREFERENCED(os_socket_mem_size);
				#endif
			}

			//------------------------------------------------------------------------
			void DestroySocket(void* p_os_socket_mem)
			{
				#if FRAMEPRO_SOCKETS_ENABLED
					SOCKET& socket = GetOSSocket(p_os_socket_mem);
					socket.~SOCKET();
					socket = INVALID_SOCKET;
				#else
					FRAMEPRO_UNREFERENCED(p_os_socket_mem);
				#endif
			}

			//------------------------------------------------------------------------
			void DisconnectSocket(void* p_os_socket_mem, bool stop_listening)
			{
				#if FRAMEPRO_SOCKETS_ENABLED
					SOCKET& socket = GetOSSocket(p_os_socket_mem);

					if(socket != INVALID_SOCKET)
					{
						if(!stop_listening && shutdown(socket, SD_BOTH) == SOCKET_ERROR)
							HandleSocketError();

						// loop until the socket is closed to ensure all data is sent
						unsigned int buffer = 0;
						size_t ret = 0;
						do { ret = recv(socket, (char*)&buffer, sizeof(buffer), 0); } while(ret != 0 && ret != (size_t)SOCKET_ERROR);

						if(closesocket(socket) == SOCKET_ERROR)
							HandleSocketError();

						socket = INVALID_SOCKET;
					}
				#else
					FRAMEPRO_UNREFERENCED(p_os_socket_mem);
					FRAMEPRO_UNREFERENCED(stop_listening);
				#endif
			}

			//------------------------------------------------------------------------
			bool StartSocketListening(void* p_os_socket_mem)
			{
				#if FRAMEPRO_SOCKETS_ENABLED
					return listen(GetOSSocket(p_os_socket_mem), 16) != SOCKET_ERROR; //@EPIC BEGIN 16 is SOMAXCONN //@EPIC END
				#else
					FRAMEPRO_UNREFERENCED(p_os_socket_mem);
					return false;
				#endif
			}

			//------------------------------------------------------------------------
			bool BindSocket(void* p_os_socket_mem, const char* p_port)
			{
				#if FRAMEPRO_SOCKETS_ENABLED
					// setup the addrinfo struct
					addrinfo info;
					ZeroMemory(&info, sizeof(info));
					info.ai_family = AF_INET;
					info.ai_socktype = SOCK_STREAM;
					info.ai_protocol = IPPROTO_TCP;
					info.ai_flags = AI_PASSIVE;

					// Resolve the server address and port
					addrinfo* p_result_info;
					int result = getaddrinfo(NULL, p_port, &info, &p_result_info);
					if (result != 0)
					{
						HandleSocketError();
						return false;
					}

					SOCKET& socket = GetOSSocket(p_os_socket_mem);

					socket = ::socket(
						p_result_info->ai_family,
						p_result_info->ai_socktype, 
						p_result_info->ai_protocol);

					if (socket == INVALID_SOCKET)
					{
						freeaddrinfo(p_result_info);
						HandleSocketError();
						return false;
					}

					// Setup the TCP listening socket
					result = ::bind(socket, p_result_info->ai_addr, (int)p_result_info->ai_addrlen);
					freeaddrinfo(p_result_info);

					if (result == SOCKET_ERROR)
					{
						HandleSocketError();
						DisconnectSocket(p_os_socket_mem, true);
						return false;
					}

					return true;
				#else
					FRAMEPRO_UNREFERENCED(p_os_socket_mem);
					FRAMEPRO_UNREFERENCED(p_port);
					return false;
				#endif
			}

			//------------------------------------------------------------------------
			bool AcceptSocket(void* p_source_os_socket_mem, void* p_target_os_socket_mem)
			{
				#if FRAMEPRO_SOCKETS_ENABLED
					SOCKET& source_socket = GetOSSocket(p_source_os_socket_mem);
					SOCKET& target_socket = GetOSSocket(p_target_os_socket_mem);

					target_socket = accept(source_socket, NULL, NULL);

					if (target_socket == INVALID_SOCKET)
						HandleSocketError();

					return target_socket != INVALID_SOCKET;
				#else
					FRAMEPRO_UNREFERENCED(p_source_os_socket_mem);
					FRAMEPRO_UNREFERENCED(p_target_os_socket_mem);
					return false;
				#endif
			}

			//------------------------------------------------------------------------
			bool SocketSend(void* p_os_socket_mem, const void* p_buffer, int size, int& bytes_sent)
			{
				#if FRAMEPRO_SOCKETS_ENABLED
					SOCKET& socket = GetOSSocket(p_os_socket_mem);
					bytes_sent = (int)send(socket, (char*)p_buffer, size, 0);
					return bytes_sent != SOCKET_ERROR;
				#else
					FRAMEPRO_UNREFERENCED(p_os_socket_mem);
					FRAMEPRO_UNREFERENCED(p_buffer);
					FRAMEPRO_UNREFERENCED(size);
					FRAMEPRO_UNREFERENCED(bytes_sent);
					return false;
				#endif
			}

			//------------------------------------------------------------------------
			bool SocketReceive(void* p_os_socket_mem, const void* p_buffer, int size, int& bytes_received)
			{
				#if FRAMEPRO_SOCKETS_ENABLED
					SOCKET& socket = GetOSSocket(p_os_socket_mem);
					bytes_received = (int)recv(socket, (char*)p_buffer, size, 0);
					return bytes_received != SOCKET_ERROR;
				#else
					FRAMEPRO_UNREFERENCED(p_os_socket_mem);
					FRAMEPRO_UNREFERENCED(p_buffer);
					FRAMEPRO_UNREFERENCED(size);
					FRAMEPRO_UNREFERENCED(bytes_received);
					return false;
				#endif
			}

			//------------------------------------------------------------------------
			bool IsSocketValid(const void* p_os_socket_mem)
			{
				#if FRAMEPRO_SOCKETS_ENABLED
					const SOCKET& socket = GetOSSocket(p_os_socket_mem);
					return socket != INVALID_SOCKET;
				#else
					FRAMEPRO_UNREFERENCED(p_os_socket_mem);
					return false;
				#endif
			}

			//------------------------------------------------------------------------
			HANDLE& GetOSThread(void* p_os_thread_mem)
			{
				return *(HANDLE*)p_os_thread_mem;
			}

			//------------------------------------------------------------------------
			struct ThreadContext
			{
				ThreadMain mp_ThreadMain;
				void* mp_Context;
				Allocator* mp_Allocator;
			};

			//------------------------------------------------------------------------
			unsigned long WINAPI PlatformThreadMain(void* p_param)
			{
				ThreadContext* p_context = (ThreadContext*)p_param;

				int result = p_context->mp_ThreadMain(p_context->mp_Context);

				Delete(p_context->mp_Allocator, p_context);

				return result;
			}

			//------------------------------------------------------------------------
			bool CreateThread(
				void* p_os_thread_mem,
				int os_thread_mem_size,
				ThreadMain p_thread_main,
				void* p_context,
				Allocator* p_allocator)
			{
				FRAMEPRO_ASSERT(os_thread_mem_size >= sizeof(HANDLE));
				FRAMEPRO_UNREFERENCED(os_thread_mem_size);

				ThreadContext* p_thread_context = New<ThreadContext>(p_allocator);
				if (!p_thread_context)
					return false;

				p_thread_context->mp_ThreadMain = p_thread_main;
				p_thread_context->mp_Context = p_context;
				p_thread_context->mp_Allocator = p_allocator;

				GetOSThread(p_os_thread_mem) = ::CreateThread(NULL, 0, PlatformThreadMain, p_thread_context, 0, NULL);

				return true;
			}

			//------------------------------------------------------------------------
			void DestroyThread(void* p_os_thread_mem)
			{
				GetOSThread(p_os_thread_mem) = 0;
			}

			//------------------------------------------------------------------------
			void SetThreadPriority(void* p_os_thread_mem, int priority)
			{
				HANDLE& handle = GetOSThread(p_os_thread_mem);
				::SetThreadPriority(handle, priority);
			}

			//------------------------------------------------------------------------
			void SetThreadAffinity(void* p_os_thread_mem, int priority)
			{
				HANDLE& handle = GetOSThread(p_os_thread_mem);
				SetThreadAffinityMask(handle, priority);
			}

			//------------------------------------------------------------------------
			void* CreateContextSwitchRecorder(Allocator* p_allocator)
			{
				#if FRAMEPRO_PLATFORM_WIN
					return EventTraceWin32::Create(p_allocator);
				#else
					FRAMEPRO_UNREFERENCED(p_allocator);
					return NULL;
				#endif
			}

			//------------------------------------------------------------------------
			void DestroyContextSwitchRecorder(void* p_context_switch_recorder, Allocator* p_allocator)
			{
				#if FRAMEPRO_PLATFORM_WIN
					EventTraceWin32::Destroy(p_context_switch_recorder, p_allocator);
				#else
					FRAMEPRO_UNREFERENCED(p_context_switch_recorder);
					FRAMEPRO_UNREFERENCED(p_allocator);
				#endif
			}

			//------------------------------------------------------------------------
			bool StartRecordingContextSitches(
				void* p_context_switch_recorder,
				Platform::ContextSwitchCallbackFunction p_callback,
				void* p_context,
				DynamicString& error)
			{
				#if FRAMEPRO_PLATFORM_WIN
					return EventTraceWin32::Start(
						p_context_switch_recorder,
						p_callback,
						p_context,
						error);
				#else
					FRAMEPRO_UNREFERENCED(p_context_switch_recorder);
					FRAMEPRO_UNREFERENCED(p_context);
					FRAMEPRO_UNREFERENCED(error);
					return false;
				#endif
			}

			//------------------------------------------------------------------------
			void StopRecordingContextSitches(void* p_context_switch_recorder)
			{
				#if FRAMEPRO_PLATFORM_WIN
					EventTraceWin32::Stop(p_context_switch_recorder);
				#else
					FRAMEPRO_UNREFERENCED(p_context_switch_recorder);
				#endif
			}

			//------------------------------------------------------------------------
			void FlushContextSwitches(void* p_context_switch_recorder)
			{
				#if FRAMEPRO_PLATFORM_WIN
					EventTraceWin32::Flush(p_context_switch_recorder);
				#else
					FRAMEPRO_UNREFERENCED(p_context_switch_recorder);
				#endif
			}
		}
	}

#elif FRAMEPRO_LINUX_BASED_PLATFORM

	//------------------------------------------------------------------------
	#include <sys/signal.h>
	#include <sys/types.h>
	#include <unistd.h>
	#include <sys/time.h>
	#include <sched.h>
	#include <errno.h>
	#include <limits.h>
	#include <pthread.h>
	#include <inttypes.h>

	#if FRAMEPRO_SOCKETS_ENABLED
		#include <sys/socket.h>
		#include <netinet/in.h>
	#endif

	//------------------------------------------------------------------------
	namespace FramePro
	{
		//------------------------------------------------------------------------
		namespace GenericPlatform
		{
			//------------------------------------------------------------------------
			void DebugWrite(const char* p_string)
			{
				printf("%s", p_string);
			}

			//------------------------------------------------------------------------
			pthread_mutex_t& GetOSLock(void* p_os_lock_mem)
			{
				return *(pthread_mutex_t*)p_os_lock_mem;
			}

			//------------------------------------------------------------------------
			void CreateLock(void* p_os_lock_mem, int os_lock_mem_size)
			{
				FRAMEPRO_ASSERT((size_t)os_lock_mem_size >= sizeof(pthread_mutex_t));

				pthread_mutexattr_t attr;
				pthread_mutexattr_init(&attr);
				pthread_mutexattr_settype(&attr, PTHREAD_MUTEX_RECURSIVE);
				pthread_mutex_init(&GetOSLock(p_os_lock_mem), &attr);
			}

			//------------------------------------------------------------------------
			void DestroyLock(void* p_os_lock_mem)
			{
				pthread_mutex_destroy(&GetOSLock(p_os_lock_mem));
			}

			//------------------------------------------------------------------------
			void TakeLock(void* p_os_lock_mem)
			{
				pthread_mutex_lock(&GetOSLock(p_os_lock_mem));
			}

			//------------------------------------------------------------------------
			void ReleaseLock(void* p_os_lock_mem)
			{
				pthread_mutex_unlock(&GetOSLock(p_os_lock_mem));
			}

			//------------------------------------------------------------------------
			void GetLocalTime(tm* p_tm, const time_t *p_time)
			{
				tm* p_local_tm = localtime(p_time);
				*p_tm = *p_local_tm;
			}

			//------------------------------------------------------------------------
			int GetCurrentProcessId()
			{
				return getpid();
			}

			//------------------------------------------------------------------------
			void VSPrintf(char* p_buffer, size_t const buffer_size, const char* p_format, va_list arg_list)
			{
				vsnprintf(p_buffer, buffer_size, p_format, arg_list);
			}

			//------------------------------------------------------------------------
			void ToString(int value, char* p_dest, int dest_size)
			{
				snprintf(p_dest, dest_size, "%d", value);
			}

			//------------------------------------------------------------------------
			bool GetProcessName(int, char*, int)
			{
				return false;
			}

			//------------------------------------------------------------------------
			struct LinuxEvent
			{
				pthread_cond_t  m_Cond;
				pthread_mutex_t m_Mutex;
				volatile bool m_Signalled;
				bool m_AutoReset;
			};

			//------------------------------------------------------------------------
			LinuxEvent& GetOSEventHandle(void* p_os_event_mem)
			{
				return *(LinuxEvent*)p_os_event_mem;
			}

			//------------------------------------------------------------------------
			void SetEvent(void* p_os_event_mem)
			{
				LinuxEvent& linux_event = GetOSEventHandle(p_os_event_mem);
				pthread_mutex_lock(&linux_event.m_Mutex);
				linux_event.m_Signalled = true;
				pthread_mutex_unlock(&linux_event.m_Mutex);
				pthread_cond_signal(&linux_event.m_Cond);
			}

			//------------------------------------------------------------------------
			void CreateEventX(void* p_os_event_mem, int os_event_mem_size, bool initial_state, bool auto_reset)
			{
				FRAMEPRO_ASSERT((size_t)os_event_mem_size >= sizeof(LinuxEvent));
				new (p_os_event_mem)LinuxEvent();

				LinuxEvent& linux_event = GetOSEventHandle(p_os_event_mem);

				pthread_cond_init(&linux_event.m_Cond, NULL);
				pthread_mutex_init(&linux_event.m_Mutex, NULL);
				linux_event.m_Signalled = false;
				linux_event.m_AutoReset = auto_reset;

				if (initial_state)
					SetEvent(p_os_event_mem);
			}

			//------------------------------------------------------------------------
			void DestroyEvent(void* p_os_event_mem)
			{
				LinuxEvent& linux_event = GetOSEventHandle(p_os_event_mem);
				pthread_mutex_destroy(&linux_event.m_Mutex);
				pthread_cond_destroy(&linux_event.m_Cond);
				linux_event.~LinuxEvent();
			}

			//------------------------------------------------------------------------
			void ResetEvent(void* p_os_event_mem)
			{
				LinuxEvent& linux_event = GetOSEventHandle(p_os_event_mem);
				pthread_mutex_lock(&linux_event.m_Mutex);
				linux_event.m_Signalled = false;
				pthread_mutex_unlock(&linux_event.m_Mutex);
			}

			//------------------------------------------------------------------------
			int WaitEvent(void* p_os_event_mem, int timeout)
			{
				LinuxEvent& linux_event = GetOSEventHandle(p_os_event_mem);

				pthread_mutex_lock(&linux_event.m_Mutex);

				if (linux_event.m_Signalled)
				{
					if (linux_event.m_AutoReset)
						linux_event.m_Signalled = false;
					pthread_mutex_unlock(&linux_event.m_Mutex);
					return true;
				}

				if (timeout == -1)
				{
					while (!linux_event.m_Signalled)
						pthread_cond_wait(&linux_event.m_Cond, &linux_event.m_Mutex);

					if (linux_event.m_AutoReset)
						linux_event.m_Signalled = false;

					pthread_mutex_unlock(&linux_event.m_Mutex);

					return true;
				}
				else
				{
					timeval curr;
					gettimeofday(&curr, NULL);

					timespec time;
					time.tv_sec = curr.tv_sec + timeout / 1000;
					time.tv_nsec = (curr.tv_usec * 1000) + ((timeout % 1000) * 1000000);

					time.tv_sec += time.tv_nsec / 1000000000L;
					time.tv_nsec = time.tv_nsec % 1000000000L;

					int ret = 0;
					do
					{
						ret = pthread_cond_timedwait(&linux_event.m_Cond, &linux_event.m_Mutex, &time);

					} while (!linux_event.m_Signalled && ret != ETIMEDOUT);

					if (linux_event.m_Signalled)
					{
						if (linux_event.m_AutoReset)
							linux_event.m_Signalled = false;

						pthread_mutex_unlock(&linux_event.m_Mutex);
						return true;
					}

					pthread_mutex_unlock(&linux_event.m_Mutex);
					return false;
				}
			}

			//------------------------------------------------------------------------
			#if FRAMEPRO_SOCKETS_ENABLED
				int& GetOSSocket(void* p_os_socket_mem)
				{
					return *(int*)p_os_socket_mem;
				}
			#endif

			//------------------------------------------------------------------------
			#if FRAMEPRO_SOCKETS_ENABLED
				const int& GetOSSocket(const void* p_os_socket_mem)
				{
					return *(int*)p_os_socket_mem;
				}
			#endif

			//------------------------------------------------------------------------
			#if FRAMEPRO_SOCKETS_ENABLED
				static const int g_InvalidSocketId = -1;
				static const int g_SocketErrorId = -1;
			#endif

			//------------------------------------------------------------------------
			void CreateSocket(void* p_os_socket_mem, int os_socket_mem_size)
			{
				#if FRAMEPRO_SOCKETS_ENABLED
					FRAMEPRO_ASSERT((size_t)os_socket_mem_size >= sizeof(int));
					GetOSSocket(p_os_socket_mem) = g_InvalidSocketId;
				#endif
			}

			//------------------------------------------------------------------------
			void DestroySocket(void* p_os_socket_mem)
			{
				#if FRAMEPRO_SOCKETS_ENABLED
					GetOSSocket(p_os_socket_mem) = g_InvalidSocketId;
				#endif
			}

			//------------------------------------------------------------------------
			bool InitialiseSocketSystem()
			{
				return true;
			}

			//------------------------------------------------------------------------
			void UninitialiseSocketSystem()
			{
				// do nothing
			}

			//------------------------------------------------------------------------
			void HandleSocketError()
			{
				DebugWrite("Socket Error");
			}

			//------------------------------------------------------------------------
			void DisconnectSocket(void* p_os_socket_mem, bool stop_listening)
			{
				#if FRAMEPRO_SOCKETS_ENABLED
					int& socket = GetOSSocket(p_os_socket_mem);

					if (socket != g_InvalidSocketId)
					{
						if (shutdown(socket, SHUT_RDWR) == g_SocketErrorId)
							HandleSocketError();

						// loop until the socket is closed to ensure all data is sent
						unsigned int buffer = 0;
						size_t ret = 0;
						do { ret = recv(socket, (char*)&buffer, sizeof(buffer), 0); } while (ret != 0 && ret != (size_t)g_SocketErrorId);

						close(socket);

						socket = g_InvalidSocketId;
					}
				#endif
			}

			//------------------------------------------------------------------------
			bool StartSocketListening(void* p_os_socket_mem)
			{
				#if FRAMEPRO_SOCKETS_ENABLED
					return listen(GetOSSocket(p_os_socket_mem), 16) != g_SocketErrorId; //@EPIC BEGIN 16 is SOMAXCONN //@EPIC END
				#else
					return false;
				#endif
			}

			//------------------------------------------------------------------------
			bool BindSocket(void* p_os_socket_mem, const char* p_port)
			{
				#if FRAMEPRO_SOCKETS_ENABLED
					int& socket = GetOSSocket(p_os_socket_mem);

					socket = ::socket(
						AF_INET,
						SOCK_STREAM,
						IPPROTO_TCP);

					if (socket == g_InvalidSocketId)
					{
						HandleSocketError();
						return false;
					}

					// Setup the TCP listening socket
					// Bind to INADDR_ANY
					sockaddr_in sa;
					sa.sin_family = AF_INET;
					sa.sin_addr.s_addr = INADDR_ANY;
					int iport = atoi(p_port);
					sa.sin_port = htons(iport);
					int result = ::bind(socket, (const sockaddr*)(&sa), sizeof(sockaddr_in));

					if (result == g_SocketErrorId)
					{
						HandleSocketError();
						DisconnectSocket(p_os_socket_mem, true);
						return false;
					}

					return true;
				#else
					return false;
				#endif
			}

			//------------------------------------------------------------------------
			bool AcceptSocket(void* p_source_os_socket_mem, void* p_target_os_socket_mem)
			{
				#if FRAMEPRO_SOCKETS_ENABLED
					int& source_socket = GetOSSocket(p_source_os_socket_mem);
					int& target_socket = GetOSSocket(p_target_os_socket_mem);

					target_socket = accept(source_socket, NULL, NULL);
					return target_socket != g_InvalidSocketId;
				#else
					return false;
				#endif
			}

			//------------------------------------------------------------------------
			bool SocketSend(void* p_os_socket_mem, const void* p_buffer, int size, int& bytes_sent)
			{
				#if FRAMEPRO_SOCKETS_ENABLED
					int& socket = GetOSSocket(p_os_socket_mem);
					#if FRAMEPRO_PLATFORM_LINUX || FRAMEPRO_PLATFORM_ANDROID
						int flags = MSG_NOSIGNAL;
					#else
						int flags = 0;
					#endif
					bytes_sent = (int)send(socket, (char*)p_buffer, size, flags);
					return bytes_sent != g_SocketErrorId;
				#else
					return false;
				#endif
			}

			//------------------------------------------------------------------------
			bool SocketReceive(void* p_os_socket_mem, const void* p_buffer, int size, int& bytes_received)
			{
				#if FRAMEPRO_SOCKETS_ENABLED
					int& socket = GetOSSocket(p_os_socket_mem);
					bytes_received = (int)recv(socket, (char*)p_buffer, size, 0);
					return bytes_received != g_SocketErrorId;
				#else
					return false;
				#endif
			}

			//------------------------------------------------------------------------
			bool IsSocketValid(const void* p_os_socket_mem)
			{
				#if FRAMEPRO_SOCKETS_ENABLED
					const int& socket = GetOSSocket(p_os_socket_mem);
					return socket != g_InvalidSocketId;
				#else
					return false;
				#endif
			}

			//------------------------------------------------------------------------
			struct ThreadContext
			{
				ThreadMain mp_ThreadMain;
				void* mp_Context;
				Allocator* mp_Allocator;
			};

			//------------------------------------------------------------------------
			void* PlatformThreadMain(void* p_param)
			{
				ThreadContext* p_context = (ThreadContext*)p_param;

				p_context->mp_ThreadMain(p_context->mp_Context);

				Delete(p_context->mp_Allocator, p_context);

				return NULL;
			}

			//------------------------------------------------------------------------
			pthread_t& GetOSThread(void* p_os_thread_mem)
			{
				return *(pthread_t*)p_os_thread_mem;
			}

			//------------------------------------------------------------------------
			bool CreateThread(
				void* p_os_thread_mem,
				int os_thread_mem_size,
				ThreadMain p_thread_main,
				void* p_context,
				Allocator* p_allocator)
			{
				FRAMEPRO_ASSERT((size_t)os_thread_mem_size >= sizeof(pthread_t));

				ThreadContext* p_thread_context = New<ThreadContext>(p_allocator);
				if (!p_thread_context)
					return false;

				p_thread_context->mp_ThreadMain = p_thread_main;
				p_thread_context->mp_Context = p_context;
				p_thread_context->mp_Allocator = p_allocator;

				pthread_create(&GetOSThread(p_os_thread_mem), NULL, PlatformThreadMain, p_thread_context);

				Platform::SetThreadAffinity(p_os_thread_mem, 0xffffffff);

				return true;
			}

			//------------------------------------------------------------------------
			void DestroyThread(void*)
			{
				// do nothing
			}

			//------------------------------------------------------------------------
			void SetThreadPriority(void*, int)
			{
				// not implemented
			}

			//------------------------------------------------------------------------
			void SetThreadAffinity(void*, int)
			{
				// not implemented
			}

			//------------------------------------------------------------------------
			void* CreateContextSwitchRecorder(Allocator*)
			{
				// not implemented
				return NULL;
			}

			//------------------------------------------------------------------------
			void DestroyContextSwitchRecorder(void*, Allocator*)
			{
				// not implemented
			}

			//------------------------------------------------------------------------
			bool StartRecordingContextSitches(void*, Platform::ContextSwitchCallbackFunction, void*, DynamicString&)
			{
				// not implemented
				return false;
			}

			//------------------------------------------------------------------------
			void StopRecordingContextSitches(void*)
			{
				// not implemented
			}

			//------------------------------------------------------------------------
			void FlushContextSwitches(void*)
			{
				// not implemented
			}
		}
	}

#endif

//------------------------------------------------------------------------
//                         FRAMEPRO_PLATFORM_UE4
//------------------------------------------------------------------------
#if FRAMEPRO_PLATFORM_UE4

	// implemented in FrameProPlatformUE4.cpp - contact slynch@puredevsoftware.com for this platform

//------------------------------------------------------------------------
//                         FRAMEPRO_PLATFORM_WIN
//------------------------------------------------------------------------
#elif FRAMEPRO_PLATFORM_WIN

	//------------------------------------------------------------------------

	//------------------------------------------------------------------------
	// if both of these options are commented out it will use CaptureStackBackTrace (or backtrace on linux)
	#define FRAMEPRO_USE_STACKWALK64 0				// much slower but possibly more reliable. FRAMEPRO_USE_STACKWALK64 only implemented for x86 builds.
	#define FRAMEPRO_USE_RTLVIRTUALUNWIND 0			// reported to be faster than StackWalk64 - only available on x64 builds
	#define FRAMEPRO_USE_RTLCAPTURESTACKBACKTRACE 0	// system version of FRAMEPRO_USE_RTLVIRTUALUNWIND - only available on x64 builds

	//------------------------------------------------------------------------
	// need to include windows.h in header for QueryPerformanceCounter()
	#ifndef WIN32_LEAN_AND_MEAN
		#define WIN32_LEAN_AND_MEAN
	#endif
	#pragma warning(push)
	#pragma warning(disable:4668)
	#include <windows.h>
	#pragma warning(pop)

	//------------------------------------------------------------------------
	__int64 FramePro_QueryPerformanceCounter()
	{
		__int64 time;
		::QueryPerformanceCounter((LARGE_INTEGER*)&time);
		return time;
	}

	//------------------------------------------------------------------------
	namespace FramePro
	{
		//------------------------------------------------------------------------
		int64 Platform::GetTimerFrequency()
		{
			int64 frequency = 0;
			QueryPerformanceFrequency((LARGE_INTEGER*)&frequency);
			return frequency;
		}

		//------------------------------------------------------------------------
		void Platform::DebugBreak()
		{
			::DebugBreak();
		}

		//------------------------------------------------------------------------
		int Platform::GetCore()
		{
			return GetCurrentProcessorNumber();
		}

		//------------------------------------------------------------------------
		Platform::Enum Platform::GetPlatformEnum()
		{
			return Platform::Windows;
		}

		//------------------------------------------------------------------------
		void* Platform::CreateContextSwitchRecorder(Allocator* p_allocator)
		{
			return GenericPlatform::CreateContextSwitchRecorder(p_allocator);
		}

		//------------------------------------------------------------------------
		void Platform::DestroyContextSwitchRecorder(void* p_context_switch_recorder, Allocator* p_allocator)
		{
			GenericPlatform::DestroyContextSwitchRecorder(p_context_switch_recorder, p_allocator);
		}

		//------------------------------------------------------------------------
		bool Platform::StartRecordingContextSitches(
			void* p_context_switch_recorder,
			ContextSwitchCallbackFunction p_callback,
			void* p_context,
			DynamicString& error,
			Allocator*)
		{
			return GenericPlatform::StartRecordingContextSitches(
				p_context_switch_recorder,
				p_callback,
				p_context,
				error);
		}

		//------------------------------------------------------------------------
		void Platform::StopRecordingContextSitches(void* p_context_switch_recorder)
		{
			GenericPlatform::StopRecordingContextSitches(p_context_switch_recorder);
		}

		//------------------------------------------------------------------------
		void Platform::FlushContextSwitches(void* p_context_switch_recorder)
		{
			GenericPlatform::FlushContextSwitches(p_context_switch_recorder);
		}

		//------------------------------------------------------------------------
		void Platform::EnumerateModules(Array<ModulePacket*>& module_packets, Allocator* p_allocator)
		{
			#if FRAMEPRO_ENABLE_CALLSTACKS
				EnumModulesWindows::EnumerateModules(module_packets, p_allocator);
			#else
				FRAMEPRO_UNREFERENCED(module_packets);
				FRAMEPRO_UNREFERENCED(p_allocator);
			#endif
		}

		//------------------------------------------------------------------------
		bool Platform::GetStackTrace(void** stack, int& stack_size, unsigned int& hash)
		{
			#if FRAMEPRO_USE_STACKWALK64

				// get the context
				CONTEXT context;
				memset(&context, 0, sizeof(context));
				RtlCaptureContext(&context);

				// setup the stack frame
				STACKFRAME64 stack_frame;
				memset(&stack_frame, 0, sizeof(stack_frame));
				stack_frame.AddrPC.Mode = AddrModeFlat;
				stack_frame.AddrFrame.Mode = AddrModeFlat;
				stack_frame.AddrStack.Mode = AddrModeFlat;
				DWORD machine = IMAGE_FILE_MACHINE_IA64;
				stack_frame.AddrPC.Offset = context.Rip;
				stack_frame.AddrFrame.Offset = context.Rsp;
				stack_frame.AddrStack.Offset = context.Rbp;
				HANDLE thread = GetCurrentThread();

				static HANDLE process = GetCurrentProcess();

				stack_size = 0;
				while (StackWalk64(
					machine,
					process,
					thread,
					&stack_frame,
					&context,
					NULL,
					SymFunctionTableAccess64,
					SymGetModuleBase64,
					NULL) && stack_size < FRAMEPRO_STACK_TRACE_SIZE)
				{
					void* p = (void*)(stack_frame.AddrPC.Offset);
					stack[stack_size++] = p;
				}
				hash = GetHash(stack, stack_size);
			#elif FRAMEPRO_USE_RTLVIRTUALUNWIND
				FramePro::VirtualUnwindStackWalk(stack, FRAMEPRO_STACK_TRACE_SIZE);
				hash = GetHashAndStackSize(stack, stack_size);
			#elif FRAMEPRO_USE_RTLCAPTURESTACKBACKTRACE
				stack_size = ::RtlCaptureStackBackTrace(1, FRAMEPRO_STACK_TRACE_SIZE-1, stack, (PDWORD)&hash);
			#else
				CaptureStackBackTrace(0, FRAMEPRO_STACK_TRACE_SIZE, stack, (PDWORD)&hash);
				for (stack_size = 0; stack_size<FRAMEPRO_STACK_TRACE_SIZE; ++stack_size)
					if (!stack[stack_size])
						break;
			#endif
			return true;
		}

		//------------------------------------------------------------------------
		FILE*& GetOSFile(void* p_os_file_mem)
		{
			return *(FILE**)p_os_file_mem;
		}

		//------------------------------------------------------------------------
		FILE* GetOSFile(const void* p_os_file_mem)
		{
			return *(FILE**)p_os_file_mem;
		}

		//------------------------------------------------------------------------
		bool Platform::OpenFileForRead(void* p_os_file_mem, int os_file_mem_size, const char* p_filename)
		{
			FRAMEPRO_ASSERT(os_file_mem_size > sizeof(FILE*));
			FRAMEPRO_UNREFERENCED(os_file_mem_size);
			FILE*& p_file = GetOSFile(p_os_file_mem);
			return fopen_s(&p_file, p_filename, "rb") == 0;
		}

		//------------------------------------------------------------------------
		bool Platform::OpenFileForRead(void* p_os_file_mem, int os_file_mem_size, const wchar_t* p_filename)
		{
			FRAMEPRO_ASSERT(os_file_mem_size > sizeof(FILE*));
			FRAMEPRO_UNREFERENCED(os_file_mem_size);
			FILE*& p_file = GetOSFile(p_os_file_mem);
			return _wfopen_s(&p_file, p_filename, L"rb") == 0;
		}

		//------------------------------------------------------------------------
		bool Platform::OpenFileForWrite(void* p_os_file_mem, int os_file_mem_size, const char* p_filename)
		{
			FRAMEPRO_ASSERT(os_file_mem_size > sizeof(FILE*));
			FRAMEPRO_UNREFERENCED(os_file_mem_size);
			FILE*& p_file = GetOSFile(p_os_file_mem);
			return fopen_s(&p_file, p_filename, "wb") == 0;
		}

		//------------------------------------------------------------------------
		bool Platform::OpenFileForWrite(void* p_os_file_mem, int os_file_mem_size, const wchar_t* p_filename)
		{
			FRAMEPRO_ASSERT(os_file_mem_size > sizeof(FILE*));
			FRAMEPRO_UNREFERENCED(os_file_mem_size);
			FILE*& p_file = GetOSFile(p_os_file_mem);
			return _wfopen_s(&p_file, p_filename, L"wb") == 0;
		}

		//------------------------------------------------------------------------
		void Platform::CloseFile(void* p_os_file_mem)
		{
			FILE*& p_file = GetOSFile(p_os_file_mem);
			fclose(p_file);
			p_file = NULL;
		}

		//------------------------------------------------------------------------
		void Platform::ReadFromFile(void* p_os_file_mem, void* p_data, size_t size)
		{
			FILE* p_file = GetOSFile(p_os_file_mem);
			fread(p_data, size, 1, p_file);
		}

		//------------------------------------------------------------------------
		void Platform::WriteToFile(void* p_os_file_mem, const void* p_data, size_t size)
		{
			FILE* p_file = GetOSFile(p_os_file_mem);
			fwrite(p_data, size, 1, p_file);
		}

		//------------------------------------------------------------------------
		int Platform::GetFileSize(const void* p_os_file_mem)
		{
			FILE* p_file = GetOSFile(p_os_file_mem);
			int pos = ftell(p_file);
			fseek(p_file, 0, SEEK_END);
			int size = ftell(p_file);
			fseek(p_file, pos, SEEK_SET);
			return size;
		}

		//------------------------------------------------------------------------
		void Platform::DebugWrite(const char* p_string)
		{
			GenericPlatform::DebugWrite(p_string);
		}

		//------------------------------------------------------------------------
		void Platform::CreateLock(void* p_os_lock_mem, int os_lock_mem_size)
		{
			GenericPlatform::CreateLock(p_os_lock_mem, os_lock_mem_size);
		}

		//------------------------------------------------------------------------
		void Platform::DestroyLock(void* p_os_lock_mem)
		{
			GenericPlatform::DestroyLock(p_os_lock_mem);
		}

		//------------------------------------------------------------------------
		void Platform::TakeLock(void* p_os_lock_mem)
		{
			GenericPlatform::TakeLock(p_os_lock_mem);
		}

		//------------------------------------------------------------------------
		void Platform::ReleaseLock(void* p_os_lock_mem)
		{
			GenericPlatform::ReleaseLock(p_os_lock_mem);
		}

		//------------------------------------------------------------------------
		void Platform::GetLocalTime(tm* p_tm, const time_t *p_time)
		{
			GenericPlatform::GetLocalTime(p_tm, p_time);
		}

		//------------------------------------------------------------------------
		int Platform::GetCurrentProcessId()
		{
			return GenericPlatform::GetCurrentProcessId();
		}

		//------------------------------------------------------------------------
		void Platform::VSPrintf(char* p_buffer, size_t const buffer_size, const char* p_format, va_list arg_list)
		{
			GenericPlatform::VSPrintf(p_buffer, buffer_size, p_format, arg_list);
		}

		//------------------------------------------------------------------------
		void Platform::ToString(int value, char* p_dest, int dest_size)
		{
			GenericPlatform::ToString(value, p_dest, dest_size);
		}

		//------------------------------------------------------------------------
		int Platform::GetCurrentThreadId()
		{
			return ::GetCurrentThreadId();
		}

		//------------------------------------------------------------------------
		bool Platform::GetProcessName(int process_id, char* p_name, int max_name_length)
		{
			HANDLE process = OpenProcess(PROCESS_ALL_ACCESS, true, process_id);
			if(process)
			{
				unsigned long result = GetProcessImageFileNameA(process, p_name, max_name_length);
				CloseHandle(process);

				if(result)
				{
					int total_length = (int)strlen(p_name);
					char* p_filename = strrchr(p_name, '\\');
					if(p_filename && p_filename[1])
					{
						++p_filename;
						memmove(p_name, p_filename, p_name + total_length + 1 - p_filename);
					}

					return true;
				}
			}

			return false;
		}

		//------------------------------------------------------------------------
		void Platform::CreateEventX(void* p_os_event_mem, int os_event_mem_size, bool initial_state, bool auto_reset)
		{
			return GenericPlatform::CreateEventX(p_os_event_mem, os_event_mem_size, initial_state, auto_reset);
		}

		//------------------------------------------------------------------------
		void Platform::DestroyEvent(void* p_os_event_mem)
		{
			GenericPlatform::DestroyEvent(p_os_event_mem);
		}

		//------------------------------------------------------------------------
		void Platform::SetEvent(void* p_os_event_mem)
		{
			GenericPlatform::SetEvent(p_os_event_mem);
		}

		//------------------------------------------------------------------------
		void Platform::ResetEvent(void* p_os_event_mem)
		{
			GenericPlatform::ResetEvent(p_os_event_mem);
		}

		//------------------------------------------------------------------------
		int Platform::WaitEvent(void* p_os_event_mem, int timeout)
		{
			return GenericPlatform::WaitEvent(p_os_event_mem, timeout);
		}

		//------------------------------------------------------------------------
		bool Platform::InitialiseSocketSystem()
		{
			return GenericPlatform::InitialiseSocketSystem();
		}

		//------------------------------------------------------------------------
		void Platform::UninitialiseSocketSystem()
		{
			GenericPlatform::UninitialiseSocketSystem();
		}

		//------------------------------------------------------------------------
		void Platform::CreateSocket(void* p_os_socket_mem, int os_socket_mem_size)
		{
			GenericPlatform::CreateSocket(p_os_socket_mem, os_socket_mem_size);
		}

		//------------------------------------------------------------------------
		void Platform::DestroySocket(void* p_os_socket_mem)
		{
			GenericPlatform::DestroySocket(p_os_socket_mem);
		}

		//------------------------------------------------------------------------
		void Platform::DisconnectSocket(void* p_os_socket_mem, bool stop_listening)
		{
			GenericPlatform::DisconnectSocket(p_os_socket_mem, stop_listening);
		}

		//------------------------------------------------------------------------
		bool Platform::StartSocketListening(void* p_os_socket_mem)
		{
			return GenericPlatform::StartSocketListening(p_os_socket_mem);
		}

		//------------------------------------------------------------------------
		bool Platform::BindSocket(void* p_os_socket_mem, const char* p_port)
		{
			return GenericPlatform::BindSocket(p_os_socket_mem, p_port);
		}

		//------------------------------------------------------------------------
		bool Platform::AcceptSocket(void* p_source_os_socket_mem, void* p_target_os_socket_mem)
		{
			return GenericPlatform::AcceptSocket(p_source_os_socket_mem, p_target_os_socket_mem);
		}

		//------------------------------------------------------------------------
		bool Platform::SocketSend(void* p_os_socket_mem, const void* p_buffer, int size, int& bytes_sent)
		{
			return GenericPlatform::SocketSend(p_os_socket_mem, p_buffer, size, bytes_sent);
		}

		//------------------------------------------------------------------------
		bool Platform::SocketReceive(void* p_os_socket_mem, const void* p_buffer, int size, int& bytes_received)
		{
			return GenericPlatform::SocketReceive(p_os_socket_mem, p_buffer, size, bytes_received);
		}

		//------------------------------------------------------------------------
		bool Platform::IsSocketValid(const void* p_os_socket_mem)
		{
			return GenericPlatform::IsSocketValid(p_os_socket_mem);
		}

		//------------------------------------------------------------------------
		void Platform::HandleSocketError()
		{
			GenericPlatform::HandleSocketError();
		}

		//------------------------------------------------------------------------
		bool Platform::CreateThread(
			void* p_os_thread_mem,
			int os_thread_mem_size,
			ThreadMain p_thread_main,
			void* p_context,
			Allocator* p_allocator)
		{
			return GenericPlatform::CreateThread(
				p_os_thread_mem,
				os_thread_mem_size,
				p_thread_main,
				p_context,
				p_allocator);
		}

		//------------------------------------------------------------------------
		void Platform::DestroyThread(void* p_os_thread_mem)
		{
			GenericPlatform::DestroyThread(p_os_thread_mem);
		}

		//------------------------------------------------------------------------
		void Platform::SetThreadPriority(void* p_os_thread_mem, int priority)
		{
			GenericPlatform::SetThreadPriority(p_os_thread_mem, priority);
		}

		//------------------------------------------------------------------------
		void Platform::SetThreadAffinity(void* p_os_thread_mem, int priority)
		{
			GenericPlatform::SetThreadAffinity(p_os_thread_mem, priority);
		}

		//------------------------------------------------------------------------
		#if FRAMEPRO_USE_TLS_SLOTS
			#error this platform is not using TLS slots
		#endif

		__declspec(thread) void* gp_FrameProTLS = NULL;

		//------------------------------------------------------------------------
		uint Platform::AllocateTLSSlot()
		{
			return 0;
		}

		//------------------------------------------------------------------------
		void* Platform::GetTLSValue(uint)
		{
			return gp_FrameProTLS;
		}

		//------------------------------------------------------------------------
		void Platform::SetTLSValue(uint, void* p_value)
		{
			gp_FrameProTLS = p_value;
		}

		//------------------------------------------------------------------------
		void Platform::GetRecordingFolder(char* p_path, int max_path_length)
		{
			FRAMEPRO_ASSERT(max_path_length);
			FRAMEPRO_UNREFERENCED(max_path_length);
			*p_path = '\0';
		}
	}

//------------------------------------------------------------------------
//                         FRAMEPRO_PLATFORM_UWP
//------------------------------------------------------------------------
#elif FRAMEPRO_PLATFORM_UWP

	//------------------------------------------------------------------------
	// need to include windows.h in header for QueryPerformanceCounter()
	#ifndef WIN32_LEAN_AND_MEAN
		#define WIN32_LEAN_AND_MEAN
	#endif
	#pragma warning(push)
	#pragma warning(disable:4668)
	#include <windows.h>
	#pragma warning(pop)

	//------------------------------------------------------------------------
	__int64 FramePro_QueryPerformanceCounter()
	{
		__int64 time;
		::QueryPerformanceCounter((LARGE_INTEGER*)&time);
		return time;
	}

	//------------------------------------------------------------------------
	namespace FramePro
	{
		//------------------------------------------------------------------------
		int64 Platform::GetTimerFrequency()
		{
			int64 frequency = 0;
			QueryPerformanceFrequency((LARGE_INTEGER*)&frequency);
			return frequency;
		}

		//------------------------------------------------------------------------
		void Platform::DebugBreak()
		{
			::DebugBreak();
		}

		//------------------------------------------------------------------------
		int Platform::GetCore()
		{
			int cpu_info[4];
			__cpuid(cpu_info, 1);
			return (cpu_info[1] >> 24) & 0xff;
		}

		//------------------------------------------------------------------------
		Platform::Enum Platform::GetPlatformEnum()
		{
			return Platform::Windows_UWP;
		}

		//------------------------------------------------------------------------
		void* Platform::CreateContextSwitchRecorder(Allocator*)
		{
			return NULL;
		}

		//------------------------------------------------------------------------
		void Platform::DestroyContextSwitchRecorder(void*, Allocator*)
		{
		}

		//------------------------------------------------------------------------
		bool Platform::StartRecordingContextSitches(
			void*,
			ContextSwitchCallbackFunction,
			void*,
			DynamicString&,
			Allocator*)
		{
			FramePro::DebugWrite("FramePro Warning: Failed to start recording context switches. Context switches may not be supported for this platform\n");
			return false;
		}

		//------------------------------------------------------------------------
		void Platform::StopRecordingContextSitches(void*)
		{
		}

		//------------------------------------------------------------------------
		void Platform::FlushContextSwitches(void*)
		{
		}

		//------------------------------------------------------------------------
		void Platform::EnumerateModules(Array<ModulePacket*>& module_packets, Allocator* p_allocator)
		{
			#if FRAMEPRO_ENABLE_CALLSTACKS
				EnumModulesWindows::EnumerateModules(module_packets, p_allocator);
			#endif
		}

		//------------------------------------------------------------------------
		bool Platform::GetStackTrace(void**, int&, unsigned int&)
		{
			return false;
		}

		//------------------------------------------------------------------------
		FILE*& GetOSFile(void* p_os_file_mem)
		{
			return *(FILE**)p_os_file_mem;
		}

		//------------------------------------------------------------------------
		FILE* GetOSFile(const void* p_os_file_mem)
		{
			return *(FILE**)p_os_file_mem;
		}

		//------------------------------------------------------------------------
		bool Platform::OpenFileForRead(void* p_os_file_mem, int os_file_mem_size, const char* p_filename)
		{
			FRAMEPRO_ASSERT(os_file_mem_size > sizeof(FILE*));
			FILE*& p_file = GetOSFile(p_os_file_mem);
			return fopen_s(&p_file, p_filename, "rb") == 0;
		}

		//------------------------------------------------------------------------
		bool Platform::OpenFileForRead(void* p_os_file_mem, int os_file_mem_size, const wchar_t* p_filename)
		{
			FRAMEPRO_ASSERT(os_file_mem_size > sizeof(FILE*));
			FILE*& p_file = GetOSFile(p_os_file_mem);
			return _wfopen_s(&p_file, p_filename, L"rb") == 0;
		}

		//------------------------------------------------------------------------
		bool Platform::OpenFileForWrite(void* p_os_file_mem, int os_file_mem_size, const char* p_filename)
		{
			FRAMEPRO_ASSERT(os_file_mem_size > sizeof(FILE*));
			FILE*& p_file = GetOSFile(p_os_file_mem);
			return fopen_s(&p_file, p_filename, "wb") == 0;
		}

		//------------------------------------------------------------------------
		bool Platform::OpenFileForWrite(void* p_os_file_mem, int os_file_mem_size, const wchar_t* p_filename)
		{
			FRAMEPRO_ASSERT(os_file_mem_size > sizeof(FILE*));
			FILE*& p_file = GetOSFile(p_os_file_mem);
			return _wfopen_s(&p_file, p_filename, L"wb") == 0;
		}

		//------------------------------------------------------------------------
		void Platform::CloseFile(void* p_os_file_mem)
		{
			FILE*& p_file = GetOSFile(p_os_file_mem);
			fclose(p_file);
			p_file = NULL;
		}

		//------------------------------------------------------------------------
		void Platform::ReadFromFile(void* p_os_file_mem, void* p_data, size_t size)
		{
			FILE* p_file = GetOSFile(p_os_file_mem);
			fread(p_data, size, 1, p_file);
		}

		//------------------------------------------------------------------------
		void Platform::WriteToFile(void* p_os_file_mem, const void* p_data, size_t size)
		{
			FILE* p_file = GetOSFile(p_os_file_mem);
			fwrite(p_data, size, 1, p_file);
		}

		//------------------------------------------------------------------------
		int Platform::GetFileSize(const void* p_os_file_mem)
		{
			FILE* p_file = GetOSFile(p_os_file_mem);
			int pos = ftell(p_file);
			fseek(p_file, 0, SEEK_END);
			int size = ftell(p_file);
			fseek(p_file, pos, SEEK_SET);
			return size;
		}

		//------------------------------------------------------------------------
		void Platform::DebugWrite(const char* p_string)
		{
			GenericPlatform::DebugWrite(p_string);
		}

		//------------------------------------------------------------------------
		void Platform::CreateLock(void* p_os_lock_mem, int os_lock_mem_size)
		{
			GenericPlatform::CreateLock(p_os_lock_mem, os_lock_mem_size);
		}

		//------------------------------------------------------------------------
		void Platform::DestroyLock(void* p_os_lock_mem)
		{
			GenericPlatform::DestroyLock(p_os_lock_mem);
		}

		//------------------------------------------------------------------------
		void Platform::TakeLock(void* p_os_lock_mem)
		{
			GenericPlatform::TakeLock(p_os_lock_mem);
		}

		//------------------------------------------------------------------------
		void Platform::ReleaseLock(void* p_os_lock_mem)
		{
			GenericPlatform::ReleaseLock(p_os_lock_mem);
		}

		//------------------------------------------------------------------------
		void Platform::GetLocalTime(tm* p_tm, const time_t *p_time)
		{
			GenericPlatform::GetLocalTime(p_tm, p_time);
		}

		//------------------------------------------------------------------------
		int Platform::GetCurrentProcessId()
		{
			return GenericPlatform::GetCurrentProcessId();
		}

		//------------------------------------------------------------------------
		void Platform::VSPrintf(char* p_buffer, size_t const buffer_size, const char* p_format, va_list arg_list)
		{
			GenericPlatform::VSPrintf(p_buffer, buffer_size, p_format, arg_list);
		}

		//------------------------------------------------------------------------
		void Platform::ToString(int value, char* p_dest, int dest_size)
		{
			GenericPlatform::ToString(value, p_dest, dest_size);
		}

		//------------------------------------------------------------------------
		int Platform::GetCurrentThreadId()
		{
			return ::GetCurrentThreadId();
		}

		//------------------------------------------------------------------------
		bool Platform::GetProcessName(int process_id, char* p_name, int max_name_length)
		{
			HANDLE process = OpenProcess(PROCESS_ALL_ACCESS, true, process_id);
			if(process)
			{
				unsigned long result = GetProcessImageFileNameA(process, p_name, max_name_length);
				CloseHandle(process);

				if(result)
				{
					int total_length = (int)strlen(p_name);
					char* p_filename = strrchr(p_name, '\\');
					if(p_filename && p_filename[1])
					{
						++p_filename;
						memmove(p_name, p_filename, p_name + total_length + 1 - p_filename);
					}

					return true;
				}
			}

			return false;
		}

		//------------------------------------------------------------------------
		void Platform::CreateEventX(void* p_os_event_mem, int os_event_mem_size, bool initial_state, bool auto_reset)
		{
			return GenericPlatform::CreateEventX(p_os_event_mem, os_event_mem_size, initial_state, auto_reset);
		}

		//------------------------------------------------------------------------
		void Platform::DestroyEvent(void* p_os_event_mem)
		{
			GenericPlatform::DestroyEvent(p_os_event_mem);
		}

		//------------------------------------------------------------------------
		void Platform::SetEvent(void* p_os_event_mem)
		{
			GenericPlatform::SetEvent(p_os_event_mem);
		}

		//------------------------------------------------------------------------
		void Platform::ResetEvent(void* p_os_event_mem)
		{
			GenericPlatform::ResetEvent(p_os_event_mem);
		}

		//------------------------------------------------------------------------
		int Platform::WaitEvent(void* p_os_event_mem, int timeout)
		{
			return GenericPlatform::WaitEvent(p_os_event_mem, timeout);
		}

		//------------------------------------------------------------------------
		bool Platform::InitialiseSocketSystem()
		{
			return GenericPlatform::InitialiseSocketSystem();
		}

		//------------------------------------------------------------------------
		void Platform::UninitialiseSocketSystem()
		{
			GenericPlatform::UninitialiseSocketSystem();
		}

		//------------------------------------------------------------------------
		void Platform::CreateSocket(void* p_os_socket_mem, int os_socket_mem_size)
		{
			GenericPlatform::CreateSocket(p_os_socket_mem, os_socket_mem_size);
		}

		//------------------------------------------------------------------------
		void Platform::DestroySocket(void* p_os_socket_mem)
		{
			GenericPlatform::DestroySocket(p_os_socket_mem);
		}

		//------------------------------------------------------------------------
		void Platform::DisconnectSocket(void* p_os_socket_mem, bool stop_listening)
		{
			GenericPlatform::DisconnectSocket(p_os_socket_mem, stop_listening);
		}

		//------------------------------------------------------------------------
		bool Platform::StartSocketListening(void* p_os_socket_mem)
		{
			return GenericPlatform::StartSocketListening(p_os_socket_mem);
		}

		//------------------------------------------------------------------------
		bool Platform::BindSocket(void* p_os_socket_mem, const char* p_port)
		{
			return GenericPlatform::BindSocket(p_os_socket_mem, p_port);
		}

		//------------------------------------------------------------------------
		bool Platform::AcceptSocket(void* p_source_os_socket_mem, void* p_target_os_socket_mem)
		{
			return GenericPlatform::AcceptSocket(p_source_os_socket_mem, p_target_os_socket_mem);
		}

		//------------------------------------------------------------------------
		bool Platform::SocketSend(void* p_os_socket_mem, const void* p_buffer, int size, int& bytes_sent)
		{
			return GenericPlatform::SocketSend(p_os_socket_mem, p_buffer, size, bytes_sent);
		}

		//------------------------------------------------------------------------
		bool Platform::SocketReceive(void* p_os_socket_mem, const void* p_buffer, int size, int& bytes_received)
		{
			return GenericPlatform::SocketReceive(p_os_socket_mem, p_buffer, size, bytes_received);
		}

		//------------------------------------------------------------------------
		bool Platform::IsSocketValid(const void* p_os_socket_mem)
		{
			return GenericPlatform::IsSocketValid(p_os_socket_mem);
		}

		//------------------------------------------------------------------------
		void Platform::HandleSocketError()
		{
			GenericPlatform::HandleSocketError();
		}

		//------------------------------------------------------------------------
		bool Platform::CreateThread(
			void* p_os_thread_mem,
			int os_thread_mem_size,
			ThreadMain p_thread_main,
			void* p_context,
			Allocator* p_allocator)
		{
			return GenericPlatform::CreateThread(
				p_os_thread_mem,
				os_thread_mem_size,
				p_thread_main,
				p_context,
				p_allocator);
		}

		//------------------------------------------------------------------------
		void Platform::DestroyThread(void* p_os_thread_mem)
		{
			GenericPlatform::DestroyThread(p_os_thread_mem);
		}

		//------------------------------------------------------------------------
		void Platform::SetThreadPriority(void* p_os_thread_mem, int priority)
		{
			GenericPlatform::SetThreadPriority(p_os_thread_mem, priority);
		}

		//------------------------------------------------------------------------
		void Platform::SetThreadAffinity(void* p_os_thread_mem, int priority)
		{
			GenericPlatform::SetThreadAffinity(p_os_thread_mem, priority);
		}

		//------------------------------------------------------------------------
		#if FRAMEPRO_USE_TLS_SLOTS
			#error this platform is not using TLS slots
		#endif

		__declspec(thread) void* gp_FrameProTLS = NULL;

		//------------------------------------------------------------------------
		uint Platform::AllocateTLSSlot()
		{
			return 0;
		}

		//------------------------------------------------------------------------
		void* Platform::GetTLSValue(uint)
		{
			return gp_FrameProTLS;
		}

		//------------------------------------------------------------------------
		void Platform::SetTLSValue(uint, void* p_value)
		{
			gp_FrameProTLS = p_value;
		}

		//------------------------------------------------------------------------
		void Platform::GetRecordingFolder(char* p_path, int max_path_length)
		{
			FRAMEPRO_ASSERT(max_path_length);
			*p_path = '\0';
		}
	}

//------------------------------------------------------------------------
//                         FRAMEPRO_PLATFORM_ANDROID
//------------------------------------------------------------------------
#elif FRAMEPRO_PLATFORM_ANDROID

	//------------------------------------------------------------------------
	#include <sys/signal.h>
	#include <sys/types.h>
	#include <unistd.h>
	#include <unwind.h>
	#include <dlfcn.h>
       	
	//------------------------------------------------------------------------
	namespace FramePro
	{
		//------------------------------------------------------------------------
		struct BacktraceState
		{
			void** mp_Current;
			void** mp_End;
		};

		//------------------------------------------------------------------------
		static _Unwind_Reason_Code UnwindCallback(struct _Unwind_Context* p_context, void* p_arg)
		{
			BacktraceState* p_state = static_cast<BacktraceState*>(p_arg);
			uintptr_t p_instr_ptr = _Unwind_GetIP(p_context);
			if (p_instr_ptr)
			{
				if (p_state->mp_Current == p_state->mp_End)
					return _URC_END_OF_STACK;
				else
					*p_state->mp_Current++ = reinterpret_cast<void*>(p_instr_ptr);
			}
			return _URC_NO_REASON;
		}

		//------------------------------------------------------------------------
		size_t backtrace(void** p_buffer, size_t buffer_size)
		{
			BacktraceState state = {p_buffer, p_buffer + buffer_size};
			_Unwind_Backtrace(UnwindCallback, &state);

			return state.mp_Current - p_buffer;
		}

		//------------------------------------------------------------------------
		int64 Platform::GetTimerFrequency()
		{
			return 1000000000;
		}

		//------------------------------------------------------------------------
		void Platform::DebugBreak()
		{
			raise(SIGTRAP);
		}

		//------------------------------------------------------------------------
		int Platform::GetCore()
		{
			return sched_getcpu();
		}

		//------------------------------------------------------------------------
		Platform::Enum Platform::GetPlatformEnum()
		{
			return Platform::Android;
		}

		//------------------------------------------------------------------------
		void Platform::EnumerateModules(Array<ModulePacket*>& module_packets, Allocator* p_allocator)
		{
			EnumModulesLinux::EnumerateModules(module_packets, p_allocator);
		}

		//------------------------------------------------------------------------
		bool Platform::GetStackTrace(void** stack, int& stack_size, unsigned int& hash)
		{
			stack_size = backtrace(stack, FRAMEPRO_STACK_TRACE_SIZE);
			hash = GetHashAndStackSize(stack, stack_size);
			return true;
		}

		//------------------------------------------------------------------------
		FILE*& GetOSFile(void* p_os_file_mem)
		{
			return *(FILE**)p_os_file_mem;
		}

		//------------------------------------------------------------------------
		FILE* GetOSFile(const void* p_os_file_mem)
		{
			return *(FILE**)p_os_file_mem;
		}

		//------------------------------------------------------------------------
		bool Platform::OpenFileForRead(void* p_os_file_mem, int os_file_mem_size, const char* p_filename)
		{
			FRAMEPRO_ASSERT((size_t)os_file_mem_size > sizeof(FILE*));
			FILE*& p_file = GetOSFile(p_os_file_mem);
			p_file = fopen(p_filename, "rb");
			return p_file != NULL;
		}

		//------------------------------------------------------------------------
		bool Platform::OpenFileForRead(void* p_os_file_mem, int os_file_mem_size, const wchar_t* p_filename)
		{
			FRAMEPRO_ASSERT((size_t)os_file_mem_size > sizeof(FILE*));
			FILE*& p_file = GetOSFile(p_os_file_mem);
			char ansi_filename[FRAMEPRO_MAX_PATH];
			wcstombs(ansi_filename, p_filename, FRAMEPRO_MAX_PATH);
			p_file = fopen(ansi_filename, "rb");
			return p_file != 0;
		}

		//------------------------------------------------------------------------
		bool Platform::OpenFileForWrite(void* p_os_file_mem, int os_file_mem_size, const char* p_filename)
		{
			FRAMEPRO_ASSERT((size_t)os_file_mem_size > sizeof(FILE*));
			FILE*& p_file = GetOSFile(p_os_file_mem);
			p_file = fopen(p_filename, "wb");
			return p_file != NULL;
		}

		//------------------------------------------------------------------------
		bool Platform::OpenFileForWrite(void* p_os_file_mem, int os_file_mem_size, const wchar_t* p_filename)
		{
			FRAMEPRO_ASSERT((size_t)os_file_mem_size > sizeof(FILE*));
			FILE*& p_file = GetOSFile(p_os_file_mem);
			char ansi_filename[FRAMEPRO_MAX_PATH];
			wcstombs(ansi_filename, p_filename, FRAMEPRO_MAX_PATH);
			p_file = fopen(ansi_filename, "wb");
			return p_file != NULL;
		}

		//------------------------------------------------------------------------
		void Platform::CloseFile(void* p_os_file_mem)
		{
			FILE*& p_file = GetOSFile(p_os_file_mem);
			fclose(p_file);
			p_file = NULL;
		}

		//------------------------------------------------------------------------
		void Platform::ReadFromFile(void* p_os_file_mem, void* p_data, size_t size)
		{
			FILE* p_file = GetOSFile(p_os_file_mem);
			fread(p_data, size, 1, p_file);
		}

		//------------------------------------------------------------------------
		void Platform::WriteToFile(void* p_os_file_mem, const void* p_data, size_t size)
		{
			FILE* p_file = GetOSFile(p_os_file_mem);
			fwrite(p_data, size, 1, p_file);
		}

		//------------------------------------------------------------------------
		int Platform::GetFileSize(const void* p_os_file_mem)
		{
			FILE* p_file = GetOSFile(p_os_file_mem);
			int pos = ftell(p_file);
			fseek(p_file, 0, SEEK_END);
			int size = ftell(p_file);
			fseek(p_file, pos, SEEK_SET);
			return size;
		}

		//------------------------------------------------------------------------
		void Platform::DebugWrite(const char* p_string)
		{
			GenericPlatform::DebugWrite(p_string);
		}

		//------------------------------------------------------------------------
		void Platform::CreateLock(void* p_os_lock_mem, int os_lock_mem_size)
		{
			GenericPlatform::CreateLock(p_os_lock_mem, os_lock_mem_size);
		}

		//------------------------------------------------------------------------
		void Platform::DestroyLock(void* p_os_lock_mem)
		{
			GenericPlatform::DestroyLock(p_os_lock_mem);
		}

		//------------------------------------------------------------------------
		void Platform::TakeLock(void* p_os_lock_mem)
		{
			GenericPlatform::TakeLock(p_os_lock_mem);
		}

		//------------------------------------------------------------------------
		void Platform::ReleaseLock(void* p_os_lock_mem)
		{
			GenericPlatform::ReleaseLock(p_os_lock_mem);
		}

		//------------------------------------------------------------------------
		void Platform::GetLocalTime(tm* p_tm, const time_t *p_time)
		{
			GenericPlatform::GetLocalTime(p_tm, p_time);
		}

		//------------------------------------------------------------------------
		int Platform::GetCurrentProcessId()
		{
			return GenericPlatform::GetCurrentProcessId();
		}

		//------------------------------------------------------------------------
		void Platform::VSPrintf(char* p_buffer, size_t const buffer_size, const char* p_format, va_list arg_list)
		{
			GenericPlatform::VSPrintf(p_buffer, buffer_size, p_format, arg_list);
		}

		//------------------------------------------------------------------------
		void Platform::ToString(int value, char* p_dest, int dest_size)
		{
			GenericPlatform::ToString(value, p_dest, dest_size);
		}

		//------------------------------------------------------------------------
		int Platform::GetCurrentThreadId()
		{
			return gettid();
		}

		//------------------------------------------------------------------------
		bool Platform::GetProcessName(int process_id, char* p_name, int max_name_length)
		{
			return GenericPlatform::GetProcessName(process_id, p_name, max_name_length);
		}

		//------------------------------------------------------------------------
		void Platform::CreateEventX(void* p_os_event_mem, int os_event_mem_size, bool initial_state, bool auto_reset)
		{
			GenericPlatform::CreateEventX(p_os_event_mem, os_event_mem_size, initial_state, auto_reset);
		}

		//------------------------------------------------------------------------
		void Platform::DestroyEvent(void* p_os_event_mem)
		{
			GenericPlatform::DestroyEvent(p_os_event_mem);
		}

		//------------------------------------------------------------------------
		void Platform::SetEvent(void* p_os_event_mem)
		{
			GenericPlatform::SetEvent(p_os_event_mem);
		}

		//------------------------------------------------------------------------
		void Platform::ResetEvent(void* p_os_event_mem)
		{
			GenericPlatform::ResetEvent(p_os_event_mem);
		}

		//------------------------------------------------------------------------
		int Platform::WaitEvent(void* p_os_event_mem, int timeout)
		{
			return GenericPlatform::WaitEvent(p_os_event_mem, timeout);
		}

		//------------------------------------------------------------------------
		void Platform::CreateSocket(void* p_os_socket_mem, int os_socket_mem_size)
		{
			GenericPlatform::CreateSocket(p_os_socket_mem, os_socket_mem_size);
		}

		//------------------------------------------------------------------------
		void Platform::DestroySocket(void* p_os_socket_mem)
		{
			GenericPlatform::DestroySocket(p_os_socket_mem);
		}

		//------------------------------------------------------------------------
		bool Platform::InitialiseSocketSystem()
		{
			return GenericPlatform::InitialiseSocketSystem();
		}

		//------------------------------------------------------------------------
		void Platform::UninitialiseSocketSystem()
		{
			GenericPlatform::UninitialiseSocketSystem();
		}

		//------------------------------------------------------------------------
		void Platform::DisconnectSocket(void* p_os_socket_mem, bool stop_listening)
		{
			GenericPlatform::DisconnectSocket(p_os_socket_mem, stop_listening);
		}

		//------------------------------------------------------------------------
		bool Platform::StartSocketListening(void* p_os_socket_mem)
		{
			return GenericPlatform::StartSocketListening(p_os_socket_mem);
		}

		//------------------------------------------------------------------------
		bool Platform::BindSocket(void* p_os_socket_mem, const char* p_port)
		{
			return GenericPlatform::BindSocket(p_os_socket_mem, p_port);
		}

		//------------------------------------------------------------------------
		bool Platform::AcceptSocket(void* p_source_os_socket_mem, void* p_target_os_socket_mem)
		{
			return GenericPlatform::AcceptSocket(p_source_os_socket_mem, p_target_os_socket_mem);
		}

		//------------------------------------------------------------------------
		bool Platform::SocketSend(void* p_os_socket_mem, const void* p_buffer, int size, int& bytes_sent)
		{
			return GenericPlatform::SocketSend(p_os_socket_mem, p_buffer, size, bytes_sent);
		}

		//------------------------------------------------------------------------
		bool Platform::SocketReceive(void* p_os_socket_mem, const void* p_buffer, int size, int& bytes_received)
		{
			return GenericPlatform::SocketReceive(p_os_socket_mem, p_buffer, size, bytes_received);
		}

		//------------------------------------------------------------------------
		bool Platform::IsSocketValid(const void* p_os_socket_mem)
		{
			return GenericPlatform::IsSocketValid(p_os_socket_mem);
		}

		//------------------------------------------------------------------------
		void Platform::HandleSocketError()
		{
			GenericPlatform::HandleSocketError();
		}

		//------------------------------------------------------------------------
		bool Platform::CreateThread(
			void* p_os_thread_mem,
			int os_thread_mem_size,
			ThreadMain p_thread_main,
			void* p_context,
			Allocator* p_allocator)
		{
			return GenericPlatform::CreateThread(
				p_os_thread_mem,
				os_thread_mem_size,
				p_thread_main,
				p_context,
				p_allocator);
		}

		//------------------------------------------------------------------------
		void Platform::DestroyThread(void* p_os_thread_mem)
		{
			GenericPlatform::DestroyThread(p_os_thread_mem);
		}

		//------------------------------------------------------------------------
		void Platform::SetThreadPriority(void* p_os_thread_mem, int priority)
		{
			GenericPlatform::SetThreadPriority(p_os_thread_mem, priority);
		}

		//------------------------------------------------------------------------
		void Platform::SetThreadAffinity(void* p_os_thread_mem, int affinity)
		{
			GenericPlatform::SetThreadAffinity(p_os_thread_mem, affinity);
		}

		//------------------------------------------------------------------------
		#if FRAMEPRO_USE_TLS_SLOTS
			#error this platform is not using TLS slots
		#endif

		__thread void* gp_FrameProTLS = NULL;

		//------------------------------------------------------------------------
		uint Platform::AllocateTLSSlot()
		{
			return 0;
		}

		//------------------------------------------------------------------------
		void* Platform::GetTLSValue(uint)
		{
			return gp_FrameProTLS;
		}

		//------------------------------------------------------------------------
		void Platform::SetTLSValue(uint, void* p_value)
		{
			gp_FrameProTLS = p_value;
		}

		//------------------------------------------------------------------------
		void Platform::GetRecordingFolder(char* p_path, int max_path_length)
		{
			FRAMEPRO_ASSERT(max_path_length);
			*p_path = '\0';
		}
		
		//------------------------------------------------------------------------
		void* Platform::CreateContextSwitchRecorder(Allocator* p_allocator)
		{
			return GenericPlatform::CreateContextSwitchRecorder(p_allocator);
		}

		//------------------------------------------------------------------------
		void Platform::DestroyContextSwitchRecorder(void* p_context_switch_recorder, Allocator* p_allocator)
		{
			GenericPlatform::DestroyContextSwitchRecorder(p_context_switch_recorder, p_allocator);
		}

		//------------------------------------------------------------------------
		bool Platform::StartRecordingContextSitches(
			void* p_context_switch_recorder,
			ContextSwitchCallbackFunction p_callback,
			void* p_context,
			DynamicString& error,
			Allocator*)
		{
			return GenericPlatform::StartRecordingContextSitches(
				p_context_switch_recorder,
				p_callback,
				p_context,
				error);
		}

		//------------------------------------------------------------------------
		void Platform::StopRecordingContextSitches(void* p_context_switch_recorder)
		{
			GenericPlatform::StopRecordingContextSitches(p_context_switch_recorder);
		}

		//------------------------------------------------------------------------
		void Platform::FlushContextSwitches(void* p_context_switch_recorder)
		{
			GenericPlatform::FlushContextSwitches(p_context_switch_recorder);
		}
	}

//------------------------------------------------------------------------
//                         FRAMEPRO_PLATFORM_LINUX
//------------------------------------------------------------------------
#elif FRAMEPRO_PLATFORM_LINUX

	//------------------------------------------------------------------------
	#include <sys/signal.h>
	#include <sys/types.h>
	#include <unistd.h>
	#include <execinfo.h>
	#include <sys/syscall.h>

	//------------------------------------------------------------------------
	namespace FramePro
	{
		//------------------------------------------------------------------------
		int64 Platform::GetTimerFrequency()
		{
			return 1000000000;
		}

		//------------------------------------------------------------------------
		void Platform::DebugBreak()
		{
			raise(SIGTRAP);
		}

		//------------------------------------------------------------------------
		int Platform::GetCore()
		{
			return sched_getcpu();
		}

		//------------------------------------------------------------------------
		Platform::Enum Platform::GetPlatformEnum()
		{
			return Platform::Linux;
		}

		//------------------------------------------------------------------------
		void Platform::EnumerateModules(Array<ModulePacket*>& module_packets, Allocator* p_allocator)
		{
			EnumModulesLinux::EnumerateModules(module_packets, p_allocator);
		}

		//------------------------------------------------------------------------
		bool Platform::GetStackTrace(void** stack, int& stack_size, unsigned int& hash)
		{
			stack_size = backtrace(stack, FRAMEPRO_STACK_TRACE_SIZE);
			hash = GetHashAndStackSize(stack, stack_size);
			return true;
		}

		//------------------------------------------------------------------------
		FILE*& GetOSFile(void* p_os_file_mem)
		{
			return *(FILE**)p_os_file_mem;
		}

		//------------------------------------------------------------------------
		FILE* GetOSFile(const void* p_os_file_mem)
		{
			return *(FILE**)p_os_file_mem;
		}

		//------------------------------------------------------------------------
		bool Platform::OpenFileForRead(void* p_os_file_mem, int os_file_mem_size, const char* p_filename)
		{
			FRAMEPRO_ASSERT((size_t)os_file_mem_size > sizeof(FILE*));
			FILE*& p_file = GetOSFile(p_os_file_mem);
			p_file = fopen(p_filename, "rb");
			return p_file != NULL;
		}

		//------------------------------------------------------------------------
		bool Platform::OpenFileForRead(void* p_os_file_mem, int os_file_mem_size, const wchar_t* p_filename)
		{
			FRAMEPRO_ASSERT((size_t)os_file_mem_size > sizeof(FILE*));
			FILE*& p_file = GetOSFile(p_os_file_mem);
			char ansi_filename[FRAMEPRO_MAX_PATH];
			wcstombs(ansi_filename, p_filename, FRAMEPRO_MAX_PATH);
			p_file = fopen(ansi_filename, "rb");
			return p_file != 0;
		}

		//------------------------------------------------------------------------
		bool Platform::OpenFileForWrite(void* p_os_file_mem, int os_file_mem_size, const char* p_filename)
		{
			FRAMEPRO_ASSERT((size_t)os_file_mem_size > sizeof(FILE*));
			FILE*& p_file = GetOSFile(p_os_file_mem);
			p_file = fopen(p_filename, "wb");
			return p_file != NULL;
		}

		//------------------------------------------------------------------------
		bool Platform::OpenFileForWrite(void* p_os_file_mem, int os_file_mem_size, const wchar_t* p_filename)
		{
			FRAMEPRO_ASSERT((size_t)os_file_mem_size > sizeof(FILE*));
			FILE*& p_file = GetOSFile(p_os_file_mem);
			char ansi_filename[FRAMEPRO_MAX_PATH];
			wcstombs(ansi_filename, p_filename, FRAMEPRO_MAX_PATH);
			p_file = fopen(ansi_filename, "wb");
			return p_file != NULL;
		}

		//------------------------------------------------------------------------
		void Platform::CloseFile(void* p_os_file_mem)
		{
			FILE*& p_file = GetOSFile(p_os_file_mem);
			fclose(p_file);
			p_file = NULL;
		}

		//------------------------------------------------------------------------
		void Platform::ReadFromFile(void* p_os_file_mem, void* p_data, size_t size)
		{
			FILE* p_file = GetOSFile(p_os_file_mem);
			fread(p_data, size, 1, p_file);
		}

		//------------------------------------------------------------------------
		void Platform::WriteToFile(void* p_os_file_mem, const void* p_data, size_t size)
		{
			FILE* p_file = GetOSFile(p_os_file_mem);
			fwrite(p_data, size, 1, p_file);
		}

		//------------------------------------------------------------------------
		int Platform::GetFileSize(const void* p_os_file_mem)
		{
			FILE* p_file = GetOSFile(p_os_file_mem);
			int pos = ftell(p_file);
			fseek(p_file, 0, SEEK_END);
			int size = ftell(p_file);
			fseek(p_file, pos, SEEK_SET);
			return size;
		}

		//------------------------------------------------------------------------
		void Platform::DebugWrite(const char* p_string)
		{
			GenericPlatform::DebugWrite(p_string);
		}

		//------------------------------------------------------------------------
		void Platform::CreateLock(void* p_os_lock_mem, int os_lock_mem_size)
		{
			GenericPlatform::CreateLock(p_os_lock_mem, os_lock_mem_size);
		}

		//------------------------------------------------------------------------
		void Platform::DestroyLock(void* p_os_lock_mem)
		{
			GenericPlatform::DestroyLock(p_os_lock_mem);
		}

		//------------------------------------------------------------------------
		void Platform::TakeLock(void* p_os_lock_mem)
		{
			GenericPlatform::TakeLock(p_os_lock_mem);
		}

		//------------------------------------------------------------------------
		void Platform::ReleaseLock(void* p_os_lock_mem)
		{
			GenericPlatform::ReleaseLock(p_os_lock_mem);
		}

		//------------------------------------------------------------------------
		void Platform::GetLocalTime(tm* p_tm, const time_t *p_time)
		{
			GenericPlatform::GetLocalTime(p_tm, p_time);
		}

		//------------------------------------------------------------------------
		int Platform::GetCurrentProcessId()
		{
			return GenericPlatform::GetCurrentProcessId();
		}

		//------------------------------------------------------------------------
		void Platform::VSPrintf(char* p_buffer, size_t const buffer_size, const char* p_format, va_list arg_list)
		{
			GenericPlatform::VSPrintf(p_buffer, buffer_size, p_format, arg_list);
		}

		//------------------------------------------------------------------------
		void Platform::ToString(int value, char* p_dest, int dest_size)
		{
			GenericPlatform::ToString(value, p_dest, dest_size);
		}

		//------------------------------------------------------------------------
		int Platform::GetCurrentThreadId()
		{
			return syscall(SYS_gettid);
		}

		//------------------------------------------------------------------------
		bool Platform::GetProcessName(int process_id, char* p_name, int max_name_length)
		{
			return GenericPlatform::GetProcessName(process_id, p_name, max_name_length);
		}

		//------------------------------------------------------------------------
		void Platform::CreateEventX(void* p_os_event_mem, int os_event_mem_size, bool initial_state, bool auto_reset)
		{
			GenericPlatform::CreateEventX(p_os_event_mem, os_event_mem_size, initial_state, auto_reset);
		}

		//------------------------------------------------------------------------
		void Platform::DestroyEvent(void* p_os_event_mem)
		{
			GenericPlatform::DestroyEvent(p_os_event_mem);
		}

		//------------------------------------------------------------------------
		void Platform::SetEvent(void* p_os_event_mem)
		{
			GenericPlatform::SetEvent(p_os_event_mem);
		}

		//------------------------------------------------------------------------
		void Platform::ResetEvent(void* p_os_event_mem)
		{
			GenericPlatform::ResetEvent(p_os_event_mem);
		}

		//------------------------------------------------------------------------
		int Platform::WaitEvent(void* p_os_event_mem, int timeout)
		{
			return GenericPlatform::WaitEvent(p_os_event_mem, timeout);
		}

		//------------------------------------------------------------------------
		void Platform::CreateSocket(void* p_os_socket_mem, int os_socket_mem_size)
		{
			GenericPlatform::CreateSocket(p_os_socket_mem, os_socket_mem_size);
		}

		//------------------------------------------------------------------------
		void Platform::DestroySocket(void* p_os_socket_mem)
		{
			GenericPlatform::DestroySocket(p_os_socket_mem);
		}

		//------------------------------------------------------------------------
		bool Platform::InitialiseSocketSystem()
		{
			return GenericPlatform::InitialiseSocketSystem();
		}

		//------------------------------------------------------------------------
		void Platform::UninitialiseSocketSystem()
		{
			GenericPlatform::UninitialiseSocketSystem();
		}

		//------------------------------------------------------------------------
		void Platform::DisconnectSocket(void* p_os_socket_mem, bool stop_listening)
		{
			GenericPlatform::DisconnectSocket(p_os_socket_mem, stop_listening);
		}

		//------------------------------------------------------------------------
		bool Platform::StartSocketListening(void* p_os_socket_mem)
		{
			return GenericPlatform::StartSocketListening(p_os_socket_mem);
		}

		//------------------------------------------------------------------------
		bool Platform::BindSocket(void* p_os_socket_mem, const char* p_port)
		{
			return GenericPlatform::BindSocket(p_os_socket_mem, p_port);
		}

		//------------------------------------------------------------------------
		bool Platform::AcceptSocket(void* p_source_os_socket_mem, void* p_target_os_socket_mem)
		{
			return GenericPlatform::AcceptSocket(p_source_os_socket_mem, p_target_os_socket_mem);
		}

		//------------------------------------------------------------------------
		bool Platform::SocketSend(void* p_os_socket_mem, const void* p_buffer, int size, int& bytes_sent)
		{
			return GenericPlatform::SocketSend(p_os_socket_mem, p_buffer, size, bytes_sent);
		}

		//------------------------------------------------------------------------
		bool Platform::SocketReceive(void* p_os_socket_mem, const void* p_buffer, int size, int& bytes_received)
		{
			return GenericPlatform::SocketReceive(p_os_socket_mem, p_buffer, size, bytes_received);
		}

		//------------------------------------------------------------------------
		bool Platform::IsSocketValid(const void* p_os_socket_mem)
		{
			return GenericPlatform::IsSocketValid(p_os_socket_mem);
		}

		//------------------------------------------------------------------------
		void Platform::HandleSocketError()
		{
			GenericPlatform::HandleSocketError();
		}

		//------------------------------------------------------------------------
		bool Platform::CreateThread(
			void* p_os_thread_mem,
			int os_thread_mem_size,
			ThreadMain p_thread_main,
			void* p_context,
			Allocator* p_allocator)
		{
			return GenericPlatform::CreateThread(
				p_os_thread_mem,
				os_thread_mem_size,
				p_thread_main,
				p_context,
				p_allocator);
		}

		//------------------------------------------------------------------------
		void Platform::DestroyThread(void* p_os_thread_mem)
		{
			GenericPlatform::DestroyThread(p_os_thread_mem);
		}

		//------------------------------------------------------------------------
		void Platform::SetThreadPriority(void* p_os_thread_mem, int priority)
		{
			GenericPlatform::SetThreadPriority(p_os_thread_mem, priority);
		}

		//------------------------------------------------------------------------
		void Platform::SetThreadAffinity(void* p_os_thread_mem, int affinity)
		{
			GenericPlatform::SetThreadAffinity(p_os_thread_mem, affinity);
		}

		//------------------------------------------------------------------------
		#if FRAMEPRO_USE_TLS_SLOTS
			#error this platform is not using TLS slots
		#endif

		__thread void* gp_FrameProTLS = NULL;

		//------------------------------------------------------------------------
		uint Platform::AllocateTLSSlot()
		{
			return 0;
		}

		//------------------------------------------------------------------------
		void* Platform::GetTLSValue(uint)
		{
			return gp_FrameProTLS;
		}

		//------------------------------------------------------------------------
		void Platform::SetTLSValue(uint, void* p_value)
		{
			gp_FrameProTLS = p_value;
		}

		//------------------------------------------------------------------------
		void Platform::GetRecordingFolder(char* p_path, int max_path_length)
		{
			FRAMEPRO_ASSERT(max_path_length);
			*p_path = '\0';
		}
		
		//------------------------------------------------------------------------
		void* Platform::CreateContextSwitchRecorder(Allocator* p_allocator)
		{
			return GenericPlatform::CreateContextSwitchRecorder(p_allocator);
		}

		//------------------------------------------------------------------------
		void Platform::DestroyContextSwitchRecorder(void* p_context_switch_recorder, Allocator* p_allocator)
		{
			GenericPlatform::DestroyContextSwitchRecorder(p_context_switch_recorder, p_allocator);
		}

		//------------------------------------------------------------------------
		bool Platform::StartRecordingContextSitches(
			void* p_context_switch_recorder,
			ContextSwitchCallbackFunction p_callback,
			void* p_context,
			DynamicString& error,
			Allocator*)
		{
			return GenericPlatform::StartRecordingContextSitches(
				p_context_switch_recorder,
				p_callback,
				p_context,
				error);
		}

		//------------------------------------------------------------------------
		void Platform::StopRecordingContextSitches(void* p_context_switch_recorder)
		{
			GenericPlatform::StopRecordingContextSitches(p_context_switch_recorder);
		}

		//------------------------------------------------------------------------
		void Platform::FlushContextSwitches(void* p_context_switch_recorder)
		{
			GenericPlatform::FlushContextSwitches(p_context_switch_recorder);
		}
	}

//------------------------------------------------------------------------
//                         FRAMEPRO_PLATFORM_XBOXONE
//------------------------------------------------------------------------
#elif FRAMEPRO_PLATFORM_XBOXONE

	// implemented in FrameProXboxOne.cpp - contact slynch@puredevsoftware.com for this platform

//------------------------------------------------------------------------
//                         FRAMEPRO_PLATFORM_PS4
//------------------------------------------------------------------------
#elif FRAMEPRO_PLATFORM_PS4

	// implemented in FrameProPS4.cpp - contact slynch@puredevsoftware.com for this platform

#else

	#error Platform not defined

#endif

//------------------------------------------------------------------------
// ---
// --- FRAMEPRO PLATFORM IMPLEMENTATION END ---
// ---
//------------------------------------------------------------------------


//------------------------------------------------------------------------
#endif		// #if FRAMEPRO_ENABLED
