// Copyright Epic Games, Inc. All Rights Reserved.

#include "FramePro/FrameProProfiler.h"

#if FRAMEPRO_ENABLED

#include "CoreTypes.h"
#include "HAL/FileManager.h"
#include "HAL/IConsoleManager.h"
#include "HAL/MallocAnsi.h"
#include "HAL/ThreadManager.h"
#include "HAL/ThreadSingleton.h"
#include "Misc/App.h"
#include "Misc/EngineVersion.h"
#include "Misc/Paths.h"
#include "Runtime/Launch/Resources/Version.h"
#include "AutoRTFM.h"

DEFINE_LOG_CATEGORY_STATIC(LogFramePro, Log, All);

static int32 GFrameProEnabled = 0;
static FAutoConsoleVariableRef CVarVerboseScriptStats(
	TEXT("framepro.enabled"),
	GFrameProEnabled,
	TEXT("Enable FramePro named events.\n"),
	ECVF_Default
);

static bool GFrameProIsRecording = 0;

#if PLATFORM_TCHAR_IS_CHAR16
#define FP_TEXT_PASTE(x) L ## x
#define WTEXT(x) FP_TEXT_PASTE(x)
#else
#define WTEXT TEXT
#endif

/** One entry in timer scope stack */
struct FFrameProProfilerScope
{
	int64 StartTime;
	FramePro::StringId StatStringId;

	FFrameProProfilerScope()
		: StartTime(0)
		, StatStringId(0)
	{}

	FORCEINLINE void BeginScope()
	{
		FRAMEPRO_GET_CLOCK_COUNT(StartTime);
	}

	FORCEINLINE void BeginScope(const ANSICHAR* Text)
	{
		StatStringId = FramePro::RegisterString(Text);
		FRAMEPRO_GET_CLOCK_COUNT(StartTime);
	}

	FORCEINLINE void BeginScope(const TCHAR* Text)
	{

		StatStringId = FramePro::RegisterString(TCHAR_TO_WCHAR(Text));
		FRAMEPRO_GET_CLOCK_COUNT(StartTime);
	}

private:
	FORCEINLINE void EndScopeImpl(int64 EndTime)
	{
		int64 Duration = EndTime - StartTime;
		if (Duration < 0)
		{
			UE_LOGF(LogFramePro, Warning, "Invalid duration scope! Start:%lld End:%lld", StartTime, EndTime);
		}
		else if (FramePro::IsConnected() && (Duration > FramePro::GetConditionalScopeMinTime()))
		{
			FramePro::AddTimeSpan(StatStringId, "none", StartTime, EndTime);
		}
	}

public:
	FORCEINLINE void EndScope()
	{
		int64 EndTime;
		FRAMEPRO_GET_CLOCK_COUNT(EndTime);

		if (StartTime == 0)
		{
			UE_LOGF(LogFramePro, Warning, "EndScope called before BeginScope!");
			return;
		}

		EndScopeImpl(EndTime);
	}

	/** Allows override of scope name at point of closure */
	FORCEINLINE void EndScope(const ANSICHAR* Override)
	{
		int64 EndTime;
		FRAMEPRO_GET_CLOCK_COUNT(EndTime);

		StatStringId = FramePro::RegisterString(Override);

		EndScopeImpl(EndTime);
	}

	/** Allows override of scope name at point of closure */
	FORCEINLINE void EndScope(const TCHAR* Override)
	{
		int64 EndTime;
		FRAMEPRO_GET_CLOCK_COUNT(EndTime);

		StatStringId = FramePro::RegisterString(TCHAR_TO_WCHAR(Override));

		EndScopeImpl(EndTime);
	}
};


/** Separate memory allocator for FramePro internal memory. Allows adding profiler traces to the main allocator without causing recursion issues. */
class FrameProAllocator : public FramePro::Allocator
{
public:
	static FrameProAllocator& Get()
	{
		static FrameProAllocator Instance;
		return Instance;
	}

	FMalloc* GetBaseMalloc() { return &BaseMalloc; }

	virtual void* Alloc(size_t size) override
	{
		void* Ptr = nullptr;
		UE_AUTORTFM_OPEN
		{
			Ptr = BaseMalloc.Malloc(size, DEFAULT_ALIGNMENT);
		};

		// no-op for non-transactional code
		UE_AUTORTFM_ONABORT(=, this)
		{
			// Disable the code analysis warning that complains that Free is being passed
			// a pointer that may be null. Free explicitly handles this case already.
			this->Free(Ptr);
		};

		return Ptr;
	}
	virtual void Free(void* p) override
	{
		UE_AUTORTFM_ONCOMMIT(=, this)
		{
			BaseMalloc.Free(p);
		};
	}

private:
	FMallocAnsi BaseMalloc;
};

/** System memory allocator for TArrays. Redirects base malloc calls to the FrameProAllocator. */
class FrameProMalloc
{
public:
	static void* Malloc(SIZE_T Count, uint32 Alignment = DEFAULT_ALIGNMENT)
	{
		void* Ptr = nullptr;
		UE_AUTORTFM_OPEN
		{
			Ptr = FrameProAllocator::Get().GetBaseMalloc()->Malloc(Count, Alignment);
		};

		// no-op for non-transactional code
		UE_AUTORTFM_ONABORT(Ptr)
		{
			// Disable the code analysis warning that complains that Free is being passed
			// a pointer that may be null. Free explicitly handles this case already.
			FrameProMalloc::Free(Ptr);
		};

		return Ptr;
	}

	static void* Realloc(void* Original, SIZE_T Count, uint32 Alignment = DEFAULT_ALIGNMENT)
	{
		FMalloc* BaseMalloc = FrameProAllocator::Get().GetBaseMalloc();

		// This logic is from FMemory.inl
		if (AutoRTFM::IsClosed())
		{
			void* Ptr = FrameProMalloc::Malloc(Count, Alignment);
			if (!Ptr)
			{
				return nullptr;
			}

			if (Original)
			{
				SIZE_T OriginalCount = 0;
				UE_AUTORTFM_OPEN
				{
					BaseMalloc->GetAllocationSize(Original, OriginalCount);
				};

				SIZE_T CopyCount = FGenericPlatformMath::Min(Count, OriginalCount);
				FMemory::Memcpy(Ptr, Original, CopyCount);
				FrameProMalloc::Free(Original);
			}

			return Ptr;
		}

		return BaseMalloc->Realloc(Original, Count, Alignment);
	}

	static void Free(void* Original)
	{
		UE_AUTORTFM_ONCOMMIT(Original)
		{
			FrameProAllocator::Get().GetBaseMalloc()->Free(Original);
		};
	}
};

/** TSL storage for per-thread scope stack */
class FFrameProProfilerContext : public TThreadSingleton<FFrameProProfilerContext>
{
	friend TThreadSingleton<FFrameProProfilerContext>;

	FFrameProProfilerContext()
	: TThreadSingleton<FFrameProProfilerContext>()
	{
		const FString& ThreadName = FThreadManager::GetThreadName(ThreadId);
		if (ThreadName.Len())
		{
			FramePro::SetThreadName(TCHAR_TO_ANSI(*ThreadName));
		}
	}
	
	/** Array that represents thread of scopes */
	TArray<FFrameProProfilerScope, TSizedExternalAllocator<32, FrameProMalloc>> ProfilerScopes;

public:

	FORCEINLINE void PushScope()
	{
		ProfilerScopes.AddDefaulted();
		ProfilerScopes.Last().BeginScope();
	}

	FORCEINLINE void PushScope(const ANSICHAR* Text)
	{
		ProfilerScopes.AddDefaulted();
		ProfilerScopes.Last().BeginScope(Text);
	}

	FORCEINLINE void PushScope(const TCHAR* Text)
	{
		ProfilerScopes.AddDefaulted();
		ProfilerScopes.Last().BeginScope(Text);
	}

	FORCEINLINE void PopScope()
	{
		if (ProfilerScopes.Num() > 0)
		{
			ProfilerScopes.Last().EndScope();
			ProfilerScopes.RemoveAt(ProfilerScopes.Num() - 1);
		}
	}

	FORCEINLINE void PopScope(const ANSICHAR* Override)
	{
		if (ProfilerScopes.Num() > 0)
		{
			ProfilerScopes.Last().EndScope(Override);
			ProfilerScopes.RemoveAt(ProfilerScopes.Num() - 1);
		}
	}

	FORCEINLINE void PopScope(const TCHAR* Override)
	{
		if (ProfilerScopes.Num() > 0)
		{
			ProfilerScopes.Last().EndScope(Override);
			ProfilerScopes.RemoveAt(ProfilerScopes.Num() - 1);
		}
	}
};

static void SendCPUStats()
{
	FRAMEPRO_NAMED_SCOPE("FramePro_SendCPUStats");

#if PLATFORM_ANDROID

	int32 NumCores = FMath::Min(FAndroidMisc::NumberOfCores(), 8);
	for (int32 CoreIdx = 0; CoreIdx < NumCores; CoreIdx++)
	{
		float Freq = (float)FAndroidMisc::GetCoreFrequency(CoreIdx, FAndroidMisc::ECoreFrequencyProperty::CurrentFrequency) / 1000000.f;

		switch (CoreIdx)
		{
		case 0:
			FRAMEPRO_CUSTOM_STAT("Core0Frequency", Freq, "CPUFreq", "GHz", FRAMEPRO_COLOUR(255,255,255) );
			break;
		case 1:
			FRAMEPRO_CUSTOM_STAT("Core1Frequency", Freq, "CPUFreq", "GHz", FRAMEPRO_COLOUR(255,255,255));
			break;
		case 2:
			FRAMEPRO_CUSTOM_STAT("Core2Frequency", Freq, "CPUFreq", "GHz", FRAMEPRO_COLOUR(255,255,255));
			break;
		case 3:
			FRAMEPRO_CUSTOM_STAT("Core3Frequency", Freq, "CPUFreq", "GHz", FRAMEPRO_COLOUR(255,255,255));
			break;
		case 4:
			FRAMEPRO_CUSTOM_STAT("Core4Frequency", Freq, "CPUFreq", "GHz", FRAMEPRO_COLOUR(255,255,255));
			break;
		case 5:
			FRAMEPRO_CUSTOM_STAT("Core5Frequency", Freq, "CPUFreq", "GHz", FRAMEPRO_COLOUR(255,255,255));
			break;
		case 6:
			FRAMEPRO_CUSTOM_STAT("Core6Frequency", Freq, "CPUFreq", "GHz", FRAMEPRO_COLOUR(255,255,255));
			break;
		case 7:
			FRAMEPRO_CUSTOM_STAT("Core7Frequency", Freq, "CPUFreq", "GHz", FRAMEPRO_COLOUR(255,255,255));
			break;
		}
	}
#endif // PLATFORM_ANDROID
}

float GFrameProCPUStatsUpdateRate = 0.00100f; // default to 1000Hz (ie every frame)
static FAutoConsoleVariableRef CVarFrameProCPUStatsUpdateRate(
	TEXT("framepro.CPUStatsUpdateRate"),
	GFrameProCPUStatsUpdateRate,
	TEXT("Update rate in seconds for collecting CPU Stats (Default: 0.001)\n")
	TEXT("0 to disable."),
	ECVF_Default);

void FFrameProProfiler::Initialize()
{
	FRAMEPRO_SET_ALLOCATOR(&FrameProAllocator::Get());
}

void FFrameProProfiler::TearDown()
{
	GFrameProEnabled = false;
	FRAMEPRO_SHUTDOWN();
}

bool FFrameProProfiler::IsThreadContextReady()
{
	return FFrameProProfilerContext::TryGet() != nullptr;
}

void FFrameProProfiler::FrameStart()
{
	static bool bFirstFrame = true;
	if (bFirstFrame && GFrameProEnabled)
	{
		UE_LOGF(LogFramePro, Log, "FramePro Support Available");

		FramePro::SendSessionInfo(WTEXT(""), TCHAR_TO_WCHAR(*FString::Printf(TEXT("%d"), FEngineVersion::Current().GetChangelist())));

		FRAMEPRO_THREAD_ORDER(WTEXT("GameThread"));
		FRAMEPRO_THREAD_ORDER(WTEXT("RenderThread"));
		FRAMEPRO_THREAD_ORDER(WTEXT("RenderThread 1"));
		FRAMEPRO_THREAD_ORDER(WTEXT("RenderThread 2"));
		FRAMEPRO_THREAD_ORDER(WTEXT("RenderThread 3"));
		FRAMEPRO_THREAD_ORDER(WTEXT("RenderThread 4"));
		FRAMEPRO_THREAD_ORDER(WTEXT("RenderThread 5"));
		FRAMEPRO_THREAD_ORDER(WTEXT("RenderThread 6"));
		FRAMEPRO_THREAD_ORDER(WTEXT("RenderThread 7"));
		FRAMEPRO_THREAD_ORDER(WTEXT("RHIThread"));
		FRAMEPRO_THREAD_ORDER(WTEXT("TaskGraphThreadNP 0"));
		FRAMEPRO_THREAD_ORDER(WTEXT("TaskGraphThreadNP 1"));
		FRAMEPRO_THREAD_ORDER(WTEXT("TaskGraphThreadNP 2"));
		FRAMEPRO_THREAD_ORDER(WTEXT("TaskGraphThreadNP 3"));
		FRAMEPRO_THREAD_ORDER(WTEXT("AudioThread"));

		bFirstFrame = false;
	}

	if (GFrameProEnabled)
	{
		FramePro::FrameStart();

		static uint64 LastCollectionTime = FPlatformTime::Cycles64();
		uint64 CurrentTime = FPlatformTime::Cycles64();
		if (GFrameProCPUStatsUpdateRate > 0.0f)
		{
			bool bUpdateStats = ((FPlatformTime::ToSeconds64(CurrentTime - LastCollectionTime) >= GFrameProCPUStatsUpdateRate));
			if (bUpdateStats)
			{
				LastCollectionTime = CurrentTime;
				SendCPUStats();
			}
		}
	}
}

void FFrameProProfiler::PushEvent()
{
	if (GFrameProEnabled)
	{
		FFrameProProfilerContext::Get().PushScope();
	}
}

void FFrameProProfiler::PushEvent(const ANSICHAR* Text)
{
	if (GFrameProEnabled)
	{
		FFrameProProfilerContext::Get().PushScope(Text);
	}
}

void FFrameProProfiler::PushEvent(const TCHAR* Text)
{
	if (GFrameProEnabled)
	{
		FFrameProProfilerContext::Get().PushScope(Text);
	}
}

void FFrameProProfiler::PopEvent()
{
	if (GFrameProEnabled)
	{
		FFrameProProfilerContext::Get().PopScope();
	}
}

void FFrameProProfiler::PopEvent(const TCHAR* Override)
{
	if (GFrameProEnabled)
	{
		FFrameProProfilerContext::Get().PopScope(Override);
	}
}

void FFrameProProfiler::PopEvent(const ANSICHAR* Override)
{
	if (GFrameProEnabled)
	{
		FFrameProProfilerContext::Get().PopScope(Override);
	}
}

static int32 ScopeMinTimeMicroseconds = 25;
static FAutoConsoleVariableRef CVarScopeMinTimeMicroseconds(
	TEXT("framepro.ScopeMinTimeMicroseconds"),
	ScopeMinTimeMicroseconds,
	TEXT("Scopes with time taken below this threshold are not recorded in the FramePro capture.\n")
	TEXT(" This value is only used when starting framepro captures with framepro.startrec.")
);

void FFrameProProfiler::StartFrameProRecordingFromCommand(const TArray< FString >& Args)
{
	FString FilenameRoot = FString::Printf(TEXT("ProfilePid%d"), FPlatformProcess::GetCurrentProcessId());
	if (Args.Num() > 0 && Args[0].Len() > 0)
	{
		FilenameRoot = Args[0];
	}

	bool bAppendDateTime = true;
	if (Args.Num() > 1)
	{
		// If someone wants to use the full filename they provided, they need to set the 2nd arg to false.
		LexTryParseString(bAppendDateTime, *Args[1]);
	}

	StartFrameProRecording(FilenameRoot, ScopeMinTimeMicroseconds, bAppendDateTime);
}

FString FFrameProProfiler::StartFrameProRecording(const FString& FilenameRoot, int32 MinScopeTime, bool bAppendDateTime)
{
	if (GFrameProIsRecording)
	{
		StopFrameProRecording();
	}

	FString RelPathName = FPaths::ProfilingDir() + TEXT("FramePro/");
	bool bSuccess = IFileManager::Get().MakeDirectory(*RelPathName, true); // ensure folder exists

	FString Filename;
	if (bAppendDateTime)
	{
		Filename = FString::Printf(TEXT("%s(%s).framepro_recording"), *FilenameRoot, *FDateTime::Now().ToString(TEXT("%Y%m%d_%H%M%S")));
	}
	else
	{
		Filename = FilenameRoot;
	}
	FString OutputFilename = RelPathName + Filename;

	UE_LOGF(LogFramePro, Log, "--- Start Recording To File: %ls", *OutputFilename);
	
	FramePro::StartRecording(OutputFilename, FParse::Param(FCommandLine::Get(), TEXT("FrameproEnableContextSwitches")), 100 * 1024 * 1024); // 100 MB file
	FramePro::SetConditionalScopeMinTimeInMicroseconds(MinScopeTime);

	// Force this on, no events to record without it
	GFrameProEnabled = true;

	// Enable named events as well
	++GCycleStatsShouldEmitNamedEvents;

	// Set recording flag
	GFrameProIsRecording = true;

	return OutputFilename;
}

static FAutoConsoleCommand StartFrameProRecordCommand(
	TEXT("framepro.startrec"),
	TEXT("Start FramePro recording"),
	FConsoleCommandWithArgsDelegate::CreateStatic(&FFrameProProfiler::StartFrameProRecordingFromCommand)
);

void FFrameProProfiler::StopFrameProRecording()
{
	if (!GFrameProIsRecording)
	{
		return;
	}

	FramePro::StopRecording();

	// Disable named events
	GCycleStatsShouldEmitNamedEvents = FMath::Max(0, GCycleStatsShouldEmitNamedEvents - 1);

	// Clear recording flag
	GFrameProIsRecording = false;


	UE_LOGF(LogFramePro, Log, "--- Stop Recording");
}

static FAutoConsoleCommand StopFrameProRecordCommand(
	TEXT("framepro.stoprec"),
	TEXT("Stop FramePro recording"),
	FConsoleCommandDelegate::CreateStatic(&FFrameProProfiler::StopFrameProRecording)
);

bool FFrameProProfiler::IsFrameProRecording()
{
	return GFrameProIsRecording;
}



#endif // FRAMEPRO_ENABLED
