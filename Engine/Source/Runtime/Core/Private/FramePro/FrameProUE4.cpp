/*
	This software is provided 'as-is', without any express or implied warranty.
	In no event will the author(s) be held liable for any damages arising from
	the use of this software.

	Permission is granted to anyone to use this software for any purpose, including
	commercial applications, and to alter it and redistribute it freely, subject to
	the following restrictions:

	1. The origin of this software must not be misrepresented; you must not
	claim that you wrote the original software. If you use this software
	in a product, an acknowledgment in the product documentation would be
	appreciated but is not required.
	2. Altered source versions must be plainly marked as such, and must not be
	misrepresented as being the original software.
	3. This notice may not be removed or altered from any source distribution.

	Author: Stewart Lynch
	www.puredevsoftware.com
	slynch@puredevsoftware.com

	Add FramePro.cpp to your project to allow FramePro to communicate with your application.
*/
//------------------------------------------------------------------------
#ifdef __UNREAL__
#include "FramePro/FrameProUE4.h"
#include "FramePro/FramePro.h"
#else
#include "FrameProUE4.h"
#include "FramePro.h"
#endif

//------------------------------------------------------------------------
//                         FRAMEPRO_PLATFORM_UE4
//------------------------------------------------------------------------
#if FRAMEPRO_ENABLED && FRAMEPRO_PLATFORM_UE4

	//------------------------------------------------------------------------
	#include "CoreGlobals.h"
	#include "HAL/FileManager.h"
	#include "Misc/Paths.h"
	#include "HAL/PlatformProcess.h"
	#include "HAL/PlatformTLS.h"
	#include "HAL/Event.h"
	#include "HAL/CriticalSection.h"
	#include "HAL/Runnable.h"
	#include "HAL/RunnableThread.h"
	#include "HAL/PlatformStackWalk.h"
	#include "Templates/UniquePtr.h"

	//------------------------------------------------------------------------
	#if defined(_WIN32) || defined(_WIN64) || defined(WIN32) || defined(WIN64) || defined(__WIN32__) || defined(__WINDOWS__)
		// windows include needed for GetCurrentProcessorNumber()
		#include "Windows/AllowWindowsPlatformTypes.h"
		#ifndef WIN32_LEAN_AND_MEAN
			#define WIN32_LEAN_AND_MEAN
		#endif
		#pragma warning(push)
		#pragma warning(disable:4668)
		#ifndef WIN32_LEAN_AND_MEAN
			#define WIN32_LEAN_AND_MEAN
		#endif
		#include <windows.h>
		#pragma warning(pop)
		#include "Windows/HideWindowsPlatformTypes.h"
	#endif

	//------------------------------------------------------------------------
	// if both of these options are commented out it will use CaptureStackBackTrace (or backtrace on linux)
	#define FRAMEPRO_USE_STACKWALK64 0				// much slower but possibly more reliable. FRAMEPRO_USE_STACKWALK64 only implemented for x86 builds.
	#define FRAMEPRO_USE_RTLVIRTUALUNWIND 0			// reported to be faster than StackWalk64 - only available on x64 builds
	#define FRAMEPRO_USE_RTLCAPTURESTACKBACKTRACE 0	// system version of FRAMEPRO_USE_RTLVIRTUALUNWIND - only available on x64 builds

	//------------------------------------------------------------------------
	//@EPIC BEGIN: arm mac needs to use this one
	#if PLATFORM_ANDROID || PLATFORM_MAC
		#include <sys/syscall.h>
	#endif
	//@EPIC END

	//@EPIC BEGIN: workaround for -nothreading
	#if PLATFORM_WINDOWS
		#include "Windows/WindowsEvent.h"
	#elif PLATFORM_USE_PTHREADS
		#include <pthread.h>
		#include "HAL/PThreadEvent.h"
	#endif
	//@EPIC END

	//------------------------------------------------------------------------
	namespace FramePro
	{
		//------------------------------------------------------------------------
		#if FRAMEPRO_ENABLE_CALLSTACKS
			extern void BaseAddressLookupFunction();
		#endif

		//------------------------------------------------------------------------
		namespace GenericPlatform
		{
			void GetLocalTime(tm* p_tm, const time_t *p_time);
			void VSPrintf(char* p_buffer, size_t const buffer_size, const char* p_format, va_list arg_list);
			void ToString(int value, char* p_dest, int);
			bool GetProcessName(int, char*, int);
			void CreateSocket(void* p_os_socket_mem, int os_socket_mem_size);
			void DestroySocket(void* p_os_socket_mem);
			void HandleSocketError();
			void DisconnectSocket(void* p_os_socket_mem, bool stop_listening);
			bool StartSocketListening(void* p_os_socket_mem);
			bool BindSocket(void* p_os_socket_mem, const char* p_port);
			bool AcceptSocket(void* p_source_os_socket_mem, void* p_target_os_socket_mem);
			bool SocketSend(void* p_os_socket_mem, const void* p_buffer, int size, int& bytes_sent);
			bool SocketReceive(void* p_os_socket_mem, const void* p_buffer, int size, int& bytes_received);
			bool IsSocketValid(const void* p_os_socket_mem);
			void SetThreadAffinity(void*, int);
			void* CreateContextSwitchRecorder(Allocator* p_allocator);
			void DestroyContextSwitchRecorder(void* p_context_switch_recorder, Allocator* p_allocator);
			bool StartRecordingContextSitches(void* p_context_switch_recorder, Platform::ContextSwitchCallbackFunction p_callback, void* p_context, DynamicString& error);
			void StopRecordingContextSitches(void* p_context_switch_recorder);
			void FlushContextSwitches(void* p_context_switch_recorder);
		}

		//------------------------------------------------------------------------
		#if FRAMEPRO_WIN_BASED_PLATFORM
			namespace EnumModulesWindows
			{
				void EnumerateModules(Array<ModulePacket*>& module_packets, Allocator* p_allocator);
			}
		#endif

		//------------------------------------------------------------------------
		int64 Platform::GetTimerFrequency()
		{
			return (int64)(1.0 / FPlatformTime::GetSecondsPerCycle64());
		}

		//------------------------------------------------------------------------
		void Platform::DebugBreak()
		{
			UE_DEBUG_BREAK();
		}

		//------------------------------------------------------------------------
		int Platform::GetCore()
		{
			return FPlatformProcess::GetCurrentCoreNumber();
		}

		//------------------------------------------------------------------------
		Platform::Enum Platform::GetPlatformEnum()
		{
			//@EPIC: allow external definition + NDA platforms removed
			#if defined(FRAMEPRO_UNREAL_PLATFORM)
				return FRAMEPRO_UNREAL_PLATFORM;
			//@EPIC end
			#elif PLATFORM_WINDOWS
				return Platform::Windows;
			#elif PLATFORM_LINUX
				return Platform::Linux;
			#elif PLATFORM_UNIX
				return Platform::Linux;
			#elif PLATFORM_ANDROID
				return Platform::Android;
			#elif PLATFORM_MAC
				return Platform::Mac;
			#elif PLATFORM_IOS
				return Platform::iOS;
			#else
				//@EPIC: begin - useful error
				#error unknown platform or FRAMEPRO_UNREAL_PLATFORM not defined
				//@EPIC: end
			#endif
		}

		//------------------------------------------------------------------------
		void* Platform::CreateContextSwitchRecorder(Allocator* p_allocator)
		{
				return GenericPlatform::CreateContextSwitchRecorder(p_allocator);
		}

		//------------------------------------------------------------------------
		void Platform::DestroyContextSwitchRecorder(void* p_context_switch_recorder, Allocator* p_allocator)
		{
				GenericPlatform::DestroyContextSwitchRecorder(p_context_switch_recorder, p_allocator);
		}

		//------------------------------------------------------------------------
		bool Platform::StartRecordingContextSitches(
			void* p_context_switch_recorder,
			ContextSwitchCallbackFunction p_callback,
			void* p_context,
			DynamicString& error,
			Allocator* p_allocator)
		{
				FRAMEPRO_UNREFERENCED(p_allocator);

				return GenericPlatform::StartRecordingContextSitches(
					p_context_switch_recorder,
					p_callback,
					p_context,
					error);
		}

		//------------------------------------------------------------------------
		void Platform::StopRecordingContextSitches(void* p_context_switch_recorder)
		{
				GenericPlatform::StopRecordingContextSitches(p_context_switch_recorder);
		}

		//------------------------------------------------------------------------
		void Platform::FlushContextSwitches(void* p_context_switch_recorder)
		{
				GenericPlatform::FlushContextSwitches(p_context_switch_recorder);
		}

		//------------------------------------------------------------------------
		#if defined(FRAMEPRO_ENUM_MODULES_PLATFORM) && FRAMEPRO_ENUMERATE_ALL_MODULES
			namespace FRAMEPRO_ENUM_MODULES_PLATFORM { void EnumerateModules(Array<ModulePacket*>& module_packets, Allocator* p_allocator); }
		#endif

		//------------------------------------------------------------------------
		#if PLATFORM_LINUX
			namespace EnumModulesLinux { void EnumerateModules(Array<ModulePacket*>& module_packets, Allocator* p_allocator); }
		#endif

		//------------------------------------------------------------------------
		void Platform::EnumerateModules(Array<ModulePacket*>& module_packets, Allocator* p_allocator)
		{
			#if FRAMEPRO_ENABLE_CALLSTACKS
				#if FRAMEPRO_WIN_BASED_PLATFORM
					EnumModulesWindows::EnumerateModules(module_packets, p_allocator);
				#elif PLATFORM_LINUX
					EnumModulesLinux::EnumerateModules(module_packets, p_allocator);
				#elif defined(FRAMEPRO_ENUM_MODULES_PLATFORM)
					FRAMEPRO_ENUM_MODULES_PLATFORM::EnumerateModules(module_packets, p_allocator);
				#else
					ModulePacket* p_module_packet = (ModulePacket*)p_allocator->Alloc(sizeof(ModulePacket));
					if (!p_module_packet)
						return;

					memset(p_module_packet, 0, sizeof(ModulePacket));

					p_module_packet->m_PacketType = PacketType::ModulePacket;
					p_module_packet->m_UseLookupFunctionForBaseAddress = 1;
					p_module_packet->m_ModuleBase = (int64)BaseAddressLookupFunction;
					FCStringAnsi::Strncpy( p_module_packet->m_SymbolFilename, TCHAR_TO_ANSI(FPlatformProcess::ExecutableName(false)), FRAMEPRO_MAX_INLINE_STRING_LENGTH-1 );

					module_packets.Add(p_module_packet);
				#endif
			#endif
		}

		//------------------------------------------------------------------------
		bool Platform::GetStackTrace(void** stack, int& stack_size, unsigned int& hash)
		{
			FPlatformStackWalk::CaptureStackBackTrace((uint64*)stack, FRAMEPRO_STACK_TRACE_SIZE - 1);

			hash = GetHashAndStackSize(stack, stack_size);

			return true;
		}

		//------------------------------------------------------------------------
		FArchive*& GetOSFile(void* p_os_file_mem)
		{
			return *(FArchive**)p_os_file_mem;
		}

		//------------------------------------------------------------------------
		FArchive* GetOSFile(const void* p_os_file_mem)
		{
			return *(FArchive**)p_os_file_mem;
		}

		//------------------------------------------------------------------------
		bool Platform::OpenFileForRead(void* p_os_file_mem, int os_file_mem_size, const char* p_filename)
		{
			FRAMEPRO_ASSERT(os_file_mem_size > sizeof(FArchive*));

			FArchive* p_archive = IFileManager::Get().CreateFileReader(ANSI_TO_TCHAR(p_filename));

			GetOSFile(p_os_file_mem) = p_archive;

			return p_archive != NULL;
		}
		
		//------------------------------------------------------------------------
		bool Platform::OpenFileForRead(void* p_os_file_mem, int os_file_mem_size, const wchar_t* p_filename)
		{
			FRAMEPRO_ASSERT(os_file_mem_size > sizeof(FArchive*));

			FArchive* p_archive = IFileManager::Get().CreateFileReader(WCHAR_TO_TCHAR(p_filename));

			GetOSFile(p_os_file_mem) = p_archive;

			return p_archive != NULL;
		}

		//------------------------------------------------------------------------
		bool Platform::OpenFileForWrite(void* p_os_file_mem, int os_file_mem_size, const char* p_filename)
		{
			FRAMEPRO_ASSERT(os_file_mem_size > sizeof(FArchive*));

			#if ALLOW_DEBUG_FILES
				FArchive* p_archive = IFileManager::Get().CreateDebugFileWriter(ANSI_TO_TCHAR(p_filename));
			#else
				FArchive* p_archive = IFileManager::Get().CreateFileWriter(ANSI_TO_TCHAR(p_filename));
			#endif

			GetOSFile(p_os_file_mem) = p_archive;

			return p_archive != NULL;
		}

		//------------------------------------------------------------------------
		bool Platform::OpenFileForWrite(void* p_os_file_mem, int os_file_mem_size, const wchar_t* p_filename)
		{
			FRAMEPRO_ASSERT(os_file_mem_size > sizeof(FArchive*));

			#if ALLOW_DEBUG_FILES
				FArchive* p_archive = IFileManager::Get().CreateDebugFileWriter(WCHAR_TO_TCHAR(p_filename));
			#else
				FArchive* p_archive = IFileManager::Get().CreateFileWriter(WCHAR_TO_TCHAR(p_filename));
			#endif

			GetOSFile(p_os_file_mem) = p_archive;

			return p_archive != NULL;
		}

		//------------------------------------------------------------------------
		void Platform::CloseFile(void* p_os_file_mem)
		{
			FArchive* p_archive = GetOSFile(p_os_file_mem);
			p_archive->Close();
			delete p_archive;
		}

		//------------------------------------------------------------------------
		void Platform::ReadFromFile(void* p_os_file_mem, void* p_data, size_t size)
		{
			GetOSFile(p_os_file_mem)->Serialize((uint8*)p_data, size);
		}

		//------------------------------------------------------------------------
		void Platform::WriteToFile(void* p_os_file_mem, const void* p_data, size_t size)
		{
			GetOSFile(p_os_file_mem)->Serialize((uint8*)p_data, size);
		}

		//------------------------------------------------------------------------
		int Platform::GetFileSize(const void* p_os_file_mem)
		{
			return (int)GetOSFile(p_os_file_mem)->TotalSize();
		}

		//------------------------------------------------------------------------
		void Platform::DebugWrite(const char* p_string)
		{
			FGenericPlatformMisc::LowLevelOutputDebugString(ANSI_TO_TCHAR(p_string));
		}

		//------------------------------------------------------------------------
		FCriticalSection& GetOSLock(void* p_os_lock_mem)
		{
			return *(FCriticalSection*)p_os_lock_mem;
		}

		//------------------------------------------------------------------------
		void Platform::CreateLock(void* p_os_lock_mem, int os_lock_mem_size)
		{
			FRAMEPRO_ASSERT(os_lock_mem_size >= sizeof(FCriticalSection));
			new (p_os_lock_mem)FCriticalSection();
		}

		//------------------------------------------------------------------------
		void Platform::DestroyLock(void* p_os_lock_mem)
		{
			GetOSLock(p_os_lock_mem).~FCriticalSection();
		}

		//------------------------------------------------------------------------
		void Platform::TakeLock(void* p_os_lock_mem)
		{
			GetOSLock(p_os_lock_mem).Lock();
		}

		//------------------------------------------------------------------------
		void Platform::ReleaseLock(void* p_os_lock_mem)
		{
			GetOSLock(p_os_lock_mem).Unlock();
		}

		//------------------------------------------------------------------------
		void Platform::GetLocalTime(tm* p_tm, const time_t *p_time)
		{
			GenericPlatform::GetLocalTime(p_tm, p_time);
		}

		//------------------------------------------------------------------------
		int Platform::GetCurrentProcessId()
		{
			return FPlatformProcess::GetCurrentProcessId();
		}

		//------------------------------------------------------------------------
		void Platform::VSPrintf(char* p_buffer, size_t const buffer_size, const char* p_format, va_list arg_list)
		{
			GenericPlatform::VSPrintf(p_buffer, buffer_size, p_format, arg_list);
		}

		//------------------------------------------------------------------------
		void Platform::ToString(int value, char* p_dest, int dest_size)
		{
			GenericPlatform::ToString(value, p_dest, dest_size);
		}

		//------------------------------------------------------------------------
		int Platform::GetCurrentThreadId()
		{
			return FPlatformTLS::GetCurrentThreadId();
		}

		//------------------------------------------------------------------------
		bool Platform::GetProcessName(int process_id, char* p_name, int max_name_length)
		{
//@EPIC BEGIN: get process name via process_id
#if PLATFORM_DESKTOP
			FString ProcessNameOrPath = FPlatformProcess::GetApplicationName((uint32)process_id);
#else
			FString ProcessNameOrPath = FPlatformProcess::ExecutableName();
#endif
			if (ProcessNameOrPath.IsEmpty())
			{
				return false;
			}

			FString ProcessName = FPaths::GetCleanFilename(ProcessNameOrPath);
			const auto p_process_name = StringCast<ANSICHAR>(*ProcessName);
//@EPIC END
			size_t length = strlen(p_process_name.Get());
			size_t max_length = max_name_length - 1;

			int copy_length = length < max_length ? length : max_length;
			FCStringAnsi::Strncpy(p_name, p_process_name.Get(), copy_length);

			return true;
		}

		//------------------------------------------------------------------------
		FEvent*& GetOSEvent(void* p_os_event_mem)
		{
			return *(FEvent**)p_os_event_mem;
		}

		//------------------------------------------------------------------------
		void Platform::CreateEventX(void* p_os_event_mem, int os_event_mem_size, bool initial_state, bool auto_reset)
		{
//@EPIC BEGIN: workaround for -nothreading
			FEvent* p_event;
			if (FPlatformProcess::SupportsMultithreading())
			{
				p_event = FPlatformProcess::GetSynchEventFromPool(!auto_reset);
			}
			else
			{
				PRAGMA_DISABLE_DEPRECATION_WARNINGS
#if PLATFORM_WINDOWS
				p_event = new FEventWin();
				p_event->Create(!auto_reset);
#elif PLATFORM_USE_PTHREADS
				p_event = new FPThreadEvent();
				p_event->Create(!auto_reset);
#else
				checkf(false, TEXT("unsupported platform for -nothreading"));
				return;
#endif
				PRAGMA_ENABLE_DEPRECATION_WARNINGS
			}
//@EPIC END
			GetOSEvent(p_os_event_mem) = p_event;

			if(initial_state)
				SetEvent(p_os_event_mem);
		}

		//------------------------------------------------------------------------
		void Platform::DestroyEvent(void* p_os_event_mem)
		{
			FEvent* p_event = GetOSEvent(p_os_event_mem);
//@EPIC BEGIN: workaround for -nothreading
			if (FPlatformProcess::SupportsMultithreading())
			{
				FPlatformProcess::ReturnSynchEventToPool(p_event);
			}
			else
			{
				delete p_event;
			}
//@EPIC END
			GetOSEvent(p_os_event_mem) = nullptr;
		}

		//------------------------------------------------------------------------
		void Platform::SetEvent(void* p_os_event_mem)
		{
			GetOSEvent(p_os_event_mem)->Trigger();
		}

		//------------------------------------------------------------------------
		void Platform::ResetEvent(void* p_os_event_mem)
		{
			GetOSEvent(p_os_event_mem)->Reset();
		}

		//------------------------------------------------------------------------
		int Platform::WaitEvent(void* p_os_event_mem, int timeout)
		{
			FEvent* p_event = GetOSEvent(p_os_event_mem);
			return (timeout==-1) ? p_event->Wait(MAX_uint32, !FramePro::IsConnected()) : p_event->Wait(timeout, !FramePro::IsConnected());
		}

		//------------------------------------------------------------------------
		bool Platform::InitialiseSocketSystem()
		{
			// do nothing
			return true;
		}

		//------------------------------------------------------------------------
		void Platform::UninitialiseSocketSystem()
		{
			// do nothing
		}

		//------------------------------------------------------------------------
		void Platform::CreateSocket(void* p_os_socket_mem, int os_socket_mem_size)
		{
			GenericPlatform::CreateSocket(p_os_socket_mem, os_socket_mem_size);
		}

		//------------------------------------------------------------------------
		void Platform::DestroySocket(void* p_os_socket_mem)
		{
			GenericPlatform::DestroySocket(p_os_socket_mem);
		}

		//------------------------------------------------------------------------
		void Platform::DisconnectSocket(void* p_os_socket_mem, bool stop_listening)
		{
			GenericPlatform::DisconnectSocket(p_os_socket_mem, stop_listening);
		}

		//------------------------------------------------------------------------
		bool Platform::StartSocketListening(void* p_os_socket_mem)
		{
			return GenericPlatform::StartSocketListening(p_os_socket_mem);
		}

		//------------------------------------------------------------------------
		bool Platform::BindSocket(void* p_os_socket_mem, const char* p_port)
		{
			return GenericPlatform::BindSocket(p_os_socket_mem, p_port);
		}

		//------------------------------------------------------------------------
		bool Platform::AcceptSocket(void* p_source_os_socket_mem, void* p_target_os_socket_mem)
		{
			return GenericPlatform::AcceptSocket(p_source_os_socket_mem, p_target_os_socket_mem);
		}

		//------------------------------------------------------------------------
		bool Platform::SocketSend(void* p_os_socket_mem, const void* p_buffer, int size, int& bytes_sent)
		{
			return GenericPlatform::SocketSend(p_os_socket_mem, p_buffer, size, bytes_sent);
		}

		//------------------------------------------------------------------------
		bool Platform::SocketReceive(void* p_os_socket_mem, const void* p_buffer, int size, int& bytes_received)
		{
			return GenericPlatform::SocketReceive(p_os_socket_mem, p_buffer, size, bytes_received);
		}

		//------------------------------------------------------------------------
		bool Platform::IsSocketValid(const void* p_os_socket_mem)
		{
			return GenericPlatform::IsSocketValid(p_os_socket_mem);
		}

		//------------------------------------------------------------------------
		void Platform::HandleSocketError()
		{
			GenericPlatform::HandleSocketError();
		}

		//------------------------------------------------------------------------
		class UE4Thread : FRunnable
		{
		public:
			UE4Thread(FramePro::ThreadMain p_thread_main, void* p_context)
			:	mp_ThreadMain(p_thread_main),
				mp_Context(p_context)
			{
//@EPIC BEGIN: workaround for -nothreading
				if (FPlatformProcess::SupportsMultithreading())
				{
					m_Runnable = TUniquePtr<FRunnableThread>(FRunnableThread::Create(this, TEXT("FramePro")));
				}
				else
				{
#if PLATFORM_WINDOWS
					CreateThread(NULL, 0, (LPTHREAD_START_ROUTINE)mp_ThreadMain, mp_Context, 0, NULL);
#elif PLATFORM_USE_PTHREADS
					typedef void* (*PthreadEntryPoint)(void* arg);
					pthread_t thread_id;
					pthread_create(&thread_id, nullptr, (PthreadEntryPoint)mp_ThreadMain, mp_Context);
#else
					checkf(false, TEXT("unsupported platform for -nothreading"));
#endif
				}
//@EPIC END
			}

			uint32 Run() override
			{
				return mp_ThreadMain(mp_Context);
			}

			void SetPriority(int priority)
			{
				m_Runnable->SetThreadPriority((EThreadPriority)priority);
			}

		private:
			TUniquePtr<FRunnableThread> m_Runnable;

			FramePro::ThreadMain mp_ThreadMain;
			void* mp_Context;
		};

		//------------------------------------------------------------------------
		UE4Thread*& GetOSThread(void* p_os_thread_mem)
		{
			return *(UE4Thread**)p_os_thread_mem;
		}

		//------------------------------------------------------------------------
		bool Platform::CreateThread(
			void* p_os_thread_mem,
			int os_thread_mem_size,
			ThreadMain p_thread_main,
			void* p_context,
			Allocator*)
		{
			FRAMEPRO_ASSERT(os_thread_mem_size >= sizeof(UE4Thread*));

			GetOSThread(p_os_thread_mem) = new UE4Thread(p_thread_main, p_context);

			return true;
		}

		//------------------------------------------------------------------------
		void Platform::DestroyThread(void* p_os_thread_mem)
		{
			delete GetOSThread(p_os_thread_mem);
		}

		//------------------------------------------------------------------------
		void Platform::SetThreadPriority(void* p_os_thread_mem, int priority)
		{
			GetOSThread(p_os_thread_mem)->SetPriority(priority);
		}

		//------------------------------------------------------------------------
		#if !FRAMEPRO_USE_TLS_SLOTS
			#error this platform must use TLS slots
		#endif

		//------------------------------------------------------------------------
		void Platform::SetThreadAffinity(void* p_os_thread_mem, int priority)
		{
			GenericPlatform::SetThreadAffinity(p_os_thread_mem, priority);
		}

		//------------------------------------------------------------------------
		uint Platform::AllocateTLSSlot()
		{
			return FPlatformTLS::AllocTlsSlot();
		}

		//------------------------------------------------------------------------
		void* Platform::GetTLSValue(uint slot)
		{
			return FPlatformTLS::GetTlsValue(slot);
		}

		//------------------------------------------------------------------------
		void Platform::SetTLSValue(uint slot, void* p_value)
		{
			FPlatformTLS::SetTlsValue(slot, p_value);
		}

		//------------------------------------------------------------------------
		void Platform::GetRecordingFolder(char* p_path, int max_path_length)
		{
			FString path = FPaths::ProfilingDir() + TEXT("FramePro/");
			const auto p_ansi_path = StringCast<ANSICHAR>(*path);
			int length = strlen(p_ansi_path.Get()) + 1;
			FRAMEPRO_ASSERT(length <= max_path_length);
			memcpy(p_path, p_ansi_path.Get(), length);
		}

		//------------------------------------------------------------------------
		void StartRecording(const FString& filename, bool context_switches, int64 max_file_size)
		{
			StartRecording(TCHAR_TO_WCHAR(*filename), context_switches, false, max_file_size);
		}
	}

#endif

