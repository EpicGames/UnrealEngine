// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenericPlatform/GenericPlatformAtomics.h"


static_assert(sizeof(FInt128) == 16, "FInt128 size must be 16 bytes.");
static_assert(alignof(FInt128) == 16, "FInt128 alignment must equals 16.");
