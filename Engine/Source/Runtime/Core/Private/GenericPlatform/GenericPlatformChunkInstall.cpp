// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenericPlatform/GenericPlatformChunkInstall.h"
#include "Containers/Ticker.h"

DEFINE_LOG_CATEGORY(LogChunkInstaller)

void FGenericPlatformChunkInstall::AsyncInit( TFunction<void(bool/*bSuccess*/)> OnInitComplete )
{
	// default implementation does nothing except trigger the complete callback on the gamethread
	ExecuteOnGameThread(UE_SOURCE_LOCATION, [OnInitComplete = MoveTemp(OnInitComplete)]()
	{
		OnInitComplete(true);
	});
}


void FGenericPlatformChunkInstall::DoNamedChunkCompleteCallbacks( const FName NamedChunk, EChunkLocation::Type Location, bool bHasSucceeded ) const
{
	DoNamedChunkCompleteCallbacks( MakeArrayView(&NamedChunk,1), Location, bHasSucceeded );
}


void FGenericPlatformChunkInstall::DoNamedChunkCompleteCallbacks( const TArrayView<const FName>& NamedChunks, EChunkLocation::Type Location, bool bHasSucceeded ) const
{
	if (NamedChunks.Num() == 0)
	{
		return;
	}

	// always defer the callback until the next gamethread tick even if we're on the gamethread already to ensure consistency
	TArray<FName> CachedNamedChunks(NamedChunks);
	ExecuteOnGameThread(UE_SOURCE_LOCATION, [this, NamedChunks = MoveTemp(CachedNamedChunks), Location, bHasSucceeded]()
	{
		bool bIsInstalled = (Location == EChunkLocation::LocalFast) || (Location == EChunkLocation::LocalSlow);

		for (const FName& NamedChunk : NamedChunks)
		{
			if (NamedChunkCompleteDelegate.IsBound())
			{
				FNamedChunkCompleteCallbackParam Param;
				Param.NamedChunk = NamedChunk;
				Param.Location = Location;
				Param.bIsInstalled = bIsInstalled;
				Param.bHasSucceeded = bHasSucceeded;

				NamedChunkCompleteDelegate.Broadcast(Param);
			}
		}
	});
}
