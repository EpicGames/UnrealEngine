// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenericPlatform/GenericPlatformCrashContext.h"
#include "Containers/StringFwd.h"
#include "Containers/Ticker.h"
#include "HAL/FileManager.h"
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/PlatformOutputDevices.h"
#include "HAL/PlatformStackWalk.h"
#include "HAL/PlatformTime.h"
#include "Internationalization/Culture.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Regex.h"
#include "Internationalization/TextLocalizationManager.h"
#include "Logging/LogScopedCategoryAndVerbosityOverride.h"
#include "Misc/App.h"
#include "Misc/Char.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreDelegates.h"
#include "Misc/EngineBuildSettings.h"
#include "Misc/EngineVersion.h"
#include "Misc/FileHelper.h"
#include "Misc/Fork.h"
#include "Misc/Guid.h"
#include "Misc/Optional.h"
#include "Misc/OutputDeviceArchiveWrapper.h"
#include "Misc/OutputDeviceHelper.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Misc/Parse.h"
#include "Misc/Paths.h"
#include "Misc/ScopeLock.h"
#include "Misc/SecureHash.h"
#include "ProfilingDebugging/TraceAuxiliary.h"
#include "Stats/Stats.h"

#ifndef NOINITCRASHREPORTER
#define NOINITCRASHREPORTER 0
#endif

#ifndef CRASH_REPORTER_WITH_ANALYTICS
#define CRASH_REPORTER_WITH_ANALYTICS 0
#endif

// Allow projects to opt out of reporting loaded plugins.
#ifndef UE_CRASH_REPORTER_WITH_LOADED_PLUGINS
#define UE_CRASH_REPORTER_WITH_LOADED_PLUGINS 1
#endif

DEFINE_LOG_CATEGORY_STATIC(LogCrashContext, Display, All);

extern CORE_API bool GIsGPUCrashed;

static TAutoConsoleVariable<bool> CVarTraceSnapshotOnCrash(
	TEXT("CrashContext.TraceSnapshotOnCrash"),
	false,
	TEXT("Execute a trace snapshot and include in the crash report."),
	ECVF_ReadOnly
);

static TAutoConsoleVariable<int32> CVarTraceSnapshotOnCrashTailSize(
	TEXT("CrashContext.TraceSnapshotOnCrashTailSize"),
	0,
	TEXT("Max limit of tail size to write to snapshot. Zero indicates no limit."),
	ECVF_ReadOnly
);

static bool GCrashContextFormatEarlyCrashLog = false;
static FAutoConsoleVariableRef CVarCrashContextFormatEarlyCrashLog(
	TEXT("CrashContext.FormatEarlyCrashLog"),
	GCrashContextFormatEarlyCrashLog,
	TEXT("If true, log lines written to the crash report log will include timestamps and category tags,	matching the format of the on-disk log.\n"
		"Only affects crashes that occur before log output devices are fully initialized, i.e. before the log backlog is disabled. (default = false)"),
	ECVF_Default
);

/**
* Output device used to write the file log's backlog in the case of an early crash before the file log is ready.
*/
class FCrashBacklogOutputDevice : public FOutputDevice
{
public:
	FCrashBacklogOutputDevice(FArchive* InArchive)
		: LogAr(InArchive)
	{
		check(InArchive);
	}

	// FOutputDevice interface
	CORE_API void Flush()
	{
		LogAr->Flush();
	}

	CORE_API void Serialize(const TCHAR* Data, ELogVerbosity::Type Verbosity, const class FName& Category)
	{
		Serialize(Data, Verbosity, Category, -1.0);
	}

	CORE_API void Serialize(const TCHAR* Data, ELogVerbosity::Type Verbosity, const class FName& Category, const double Time)
	{
		if (Verbosity != ELogVerbosity::SetColor)
		{
			FOutputDeviceHelper::FormatCastAndSerializeLine(*LogAr, Data, Verbosity, Category, Time, bSuppressEventTag, bAutoEmitLineTerminator);
		}
	}
	// End of FOutputDevice interface

private:
	FArchive* LogAr;
};

/**
 * A function-like type that creates a TStringBuilder to xml-escape a string
 */
template<int BufferSize=512>
class FXmlEscapedString : public TStringBuilderWithBuffer<TCHAR, BufferSize>
{
public:
	explicit FXmlEscapedString(FStringView Str)
	{
		FGenericCrashContext::AppendEscapedXMLString(*this, Str);
	}
};

static bool NeedsEscape(FStringView Str)
{
	for (TCHAR C : Str)
	{
		switch (C)
		{
		case TCHAR('&'):
		case TCHAR('"'):
		case TCHAR('\''):
		case TCHAR('<'):
		case TCHAR('>'):
		case TCHAR('\r'):
			return true;
		}
	}
	return false;
}

const TCHAR* AttendedStatusToString(const EUnattendedStatus Status)
{
	switch(Status)
	{
	case EUnattendedStatus::Attended: return TEXT("Attended");
	case EUnattendedStatus::Unattended: return TEXT("Unattended");
	case EUnattendedStatus::Unknown: // fallthrough
	default:
		return TEXT("Unknown");
	}
}

// Sanitize the event name string to remove characters that are used as delimiters for parsing.
FString FGPUBreadcrumbCrashData::FSerializer::Sanitize(FString const& Name)
{
	return Name.Replace(TEXT("{"), TEXT("(")).Replace(TEXT("}"), TEXT(")"));
}

// Event names include parameters, mostly numeric (e.g. "Frame 1234"), that should be ignored when computing the hash.
FString FGPUBreadcrumbCrashData::FSerializer::SanitizeForHash(FString const& Name)
{
	FString SanitizedName;
	SanitizedName.Reserve(Name.Len());
	for (TCHAR Char : Name)
	{
		if (!FChar::IsDigit(Char))
		{
			SanitizedName.AppendChar(Char);
		}
	}
	return SanitizedName;
}

CORE_API void FGPUBreadcrumbCrashData::FSerializer::BeginNode(FString const& Name, EState State)
{
	// Update hashes using the node name
	{
		FString HashName = SanitizeForHash(Name);
		FullHash.UpdateWithString(*HashName, HashName.Len());
		if (State == EState::Active)
		{
			ActiveHash.UpdateWithString(*HashName, HashName.Len());
		}
	}

	if (ChildStack.Num() > 0)
	{
		if (ChildStack.Top())
		{
			// Current node already has children, so we need a comma separator
			String.Append(TEXT(","));
		}
		else
		{
			// Current node does not yet have a child list.
			String.Append(TEXT(",{"));
			ChildStack.Top() = true;
		}
	}

	String += FString::Printf(TEXT("{{%s},%c"), *Sanitize(Name), StateChars[int32(State)]);
	ChildStack.Push(false);
}

CORE_API void FGPUBreadcrumbCrashData::FSerializer::EndNode()
{
	check(ChildStack.Num() > 0);
	if (ChildStack.Pop())
	{
		// Need to end the child list
		String.Append(TEXT("}}"));
	}
	else
	{
		// Ending a node with no children
		String.AppendChar(TEXT('}'));
	}
}

CORE_API FGPUBreadcrumbCrashData::FQueueData FGPUBreadcrumbCrashData::FSerializer::GetResult()
{
	check(ChildStack.Num() == 0);

	FGPUBreadcrumbCrashData::FQueueData Result{};

	Result.BreadcrumbString = MoveTemp(String);
	Result.FullHash         = FullHash.Finalize();
	Result.ActiveHash       = ActiveHash.Finalize();

	return Result;
}

static FGPUBreadcrumbCrashData GPUBreadcrumbsFromSharedContext(const FGPUBreadcrumbsSharedContext& Context)
{
	FGPUBreadcrumbCrashData DstData(Context.SourceName, Context.Version);

	for (uint32 QueueIdx = 0; QueueIdx < Context.NumQueues; ++QueueIdx)
	{
		const FGPUBreadcrumbsSharedContext::FQueueData& SrcQueue = Context.Queues[QueueIdx];

		FGPUBreadcrumbCrashData::FQueueData DstQueue;
		DstQueue.ActiveHash.FromString(SrcQueue.ActiveHash);
		DstQueue.FullHash.FromString(SrcQueue.FullHash);
		DstQueue.BreadcrumbString = SrcQueue.Breadcrumbs;

		DstData.Queues.Emplace(SrcQueue.QueueName, MoveTemp(DstQueue));
	}
	
	return DstData;
}

static void GPUBreadcrumbsToSharedContext(const FGPUBreadcrumbCrashData& GPUBreadcrumbs, FGPUBreadcrumbsSharedContext& OutSharedContext)
{
	FCString::Strncpy(OutSharedContext.Version, *GPUBreadcrumbs.Version, CR_MAX_GENERIC_FIELD_CHARS);
	FCString::Strncpy(OutSharedContext.SourceName, *GPUBreadcrumbs.SourceName, CR_MAX_GENERIC_FIELD_CHARS);

	OutSharedContext.NumQueues = 0;
	for (const TPair<FString, FGPUBreadcrumbCrashData::FQueueData>& SrcData : GPUBreadcrumbs.Queues)
	{
		const FGPUBreadcrumbCrashData::FQueueData& SrcBreadcrumbs = SrcData.Value;

		// Skip queues with no breadcrumb data or with too many breadcrumbs.
		if (SrcBreadcrumbs.BreadcrumbString.IsEmpty() || SrcBreadcrumbs.BreadcrumbString.Len() >= CR_MAX_GPU_BREADCRUMBS_STRING_CHARS)
		{
			continue;
		}

		FGPUBreadcrumbsSharedContext::FQueueData& DstQueue = OutSharedContext.Queues[OutSharedContext.NumQueues];
		FCString::Strncpy(DstQueue.QueueName, *SrcData.Key, CR_MAX_GENERIC_FIELD_CHARS);
		FCString::Strncpy(DstQueue.FullHash, *SrcBreadcrumbs.FullHash.ToString(), CR_MAX_GENERIC_FIELD_CHARS);
		FCString::Strncpy(DstQueue.ActiveHash, *SrcBreadcrumbs.ActiveHash.ToString(), CR_MAX_GENERIC_FIELD_CHARS);
		FCString::Strncpy(DstQueue.Breadcrumbs, *SrcBreadcrumbs.BreadcrumbString, CR_MAX_GPU_BREADCRUMBS_STRING_CHARS);

		OutSharedContext.NumQueues++;
		if (OutSharedContext.NumQueues >= CR_MAX_GPU_BREADCRUMBS_QUEUES)
		{
			break;
		}
	}
}


/*-----------------------------------------------------------------------------
	FGenericCrashContext
-----------------------------------------------------------------------------*/

const ANSICHAR* const FGenericCrashContext::CrashContextRuntimeXMLNameA = "CrashContext.runtime-xml";
const TCHAR* const FGenericCrashContext::CrashContextRuntimeXMLNameW = TEXT( "CrashContext.runtime-xml" );

const ANSICHAR* const FGenericCrashContext::CrashConfigFileNameA = "CrashReportClient.ini";
const TCHAR* const FGenericCrashContext::CrashConfigFileNameW = TEXT("CrashReportClient.ini");
const TCHAR* const FGenericCrashContext::CrashConfigExtension = TEXT(".ini");
const TCHAR* const FGenericCrashContext::ConfigSectionName = TEXT("CrashReportClient");
const TCHAR* const FGenericCrashContext::CrashConfigPurgeDays = TEXT("CrashConfigPurgeDays");
const TCHAR* const FGenericCrashContext::CrashGUIDRootPrefix = TEXT("UECC-");

const TCHAR* const FGenericCrashContext::CrashReportOptionalAttachmentsDirName = TEXT("OptionalAttachments");

const TCHAR* const FGenericCrashContext::CrashContextExtension = TEXT(".runtime-xml");
const TCHAR* const FGenericCrashContext::RuntimePropertiesTag = TEXT( "RuntimeProperties" );
const TCHAR* const FGenericCrashContext::DeploymentNameTag = TEXT( "DeploymentName" );
const TCHAR* const FGenericCrashContext::PlatformPropertiesTag = TEXT( "PlatformProperties" );
const TCHAR* const FGenericCrashContext::EngineDataTag = TEXT( "EngineData" );
const TCHAR* const FGenericCrashContext::GameDataTag = TEXT( "GameData" );
const TCHAR* const FGenericCrashContext::RequestDataTag = TEXT( "RequestData" );
const TCHAR* const FGenericCrashContext::GameNameTag = TEXT( "GameName" );
const TCHAR* const FGenericCrashContext::EnabledPluginsTag = TEXT("EnabledPlugins");
const TCHAR* const FGenericCrashContext::CrashVersionTag = TEXT("CrashVersion");
const TCHAR* const FGenericCrashContext::ExecutionGuidTag = TEXT("ExecutionGuid");
const TCHAR* const FGenericCrashContext::CrashGuidTag = TEXT("CrashGUID");
const TCHAR* const FGenericCrashContext::IsEnsureTag = TEXT("IsEnsure");
const TCHAR* const FGenericCrashContext::IsStallTag = TEXT("IsStall");
const TCHAR* const FGenericCrashContext::IsAssertTag = TEXT("IsAssert");
const TCHAR* const FGenericCrashContext::CrashTypeTag = TEXT("CrashType");
const TCHAR* const FGenericCrashContext::ErrorMessageTag = TEXT("ErrorMessage");
const TCHAR* const FGenericCrashContext::CrashReporterMessageTag = TEXT("CrashReporterMessage");
const TCHAR* const FGenericCrashContext::AttendedStatusTag = TEXT("CrashReporterMessage");
const TCHAR* const FGenericCrashContext::ProcessIdTag = TEXT("ProcessId");
const TCHAR* const FGenericCrashContext::SecondsSinceStartTag = TEXT("SecondsSinceStart");
const TCHAR* const FGenericCrashContext::BuildVersionTag = TEXT("BuildVersion");
const TCHAR* const FGenericCrashContext::CallStackTag = TEXT("CallStack");
const TCHAR* const FGenericCrashContext::PortableCallStackTag = TEXT("PCallStack");
const TCHAR* const FGenericCrashContext::PortableCallStackHashTag = TEXT("PCallStackHash");
const TCHAR* const FGenericCrashContext::IsRequestingExitTag = TEXT("IsRequestingExit");
const TCHAR* const FGenericCrashContext::LogFilePathTag = TEXT("LogFilePath");
const TCHAR* const FGenericCrashContext::IsInternalBuildTag = TEXT("IsInternalBuild");
const TCHAR* const FGenericCrashContext::IsPerforceBuildTag = TEXT("IsPerforceBuild");
const TCHAR* const FGenericCrashContext::IsWithDebugInfoTag = TEXT("IsWithDebugInfo");
const TCHAR* const FGenericCrashContext::IsSourceDistributionTag = TEXT("IsSourceDistribution");

const TCHAR* const FGenericCrashContext::UEMinidumpName = TEXT( "UEMinidump.dmp" );
const TCHAR* const FGenericCrashContext::NewLineTag = TEXT( "&nl;" );

const TCHAR* const FGenericCrashContext::CrashTypeCrash = TEXT("Crash");
const TCHAR* const FGenericCrashContext::CrashTypeAssert = TEXT("Assert");
const TCHAR* const FGenericCrashContext::CrashTypeEnsure = TEXT("Ensure");
const TCHAR* const FGenericCrashContext::CrashTypeStall = TEXT("Stall");
const TCHAR* const FGenericCrashContext::CrashTypeGPU = TEXT("GPUCrash");
const TCHAR* const FGenericCrashContext::CrashTypeHang = TEXT("Hang");
const TCHAR* const FGenericCrashContext::CrashTypeAbnormalShutdown = TEXT("AbnormalShutdown");
const TCHAR* const FGenericCrashContext::CrashTypeOutOfMemory = TEXT("OutOfMemory");
const TCHAR* const FGenericCrashContext::CrashTypeVerseRuntimeError = TEXT("VerseRuntimeError");

const TCHAR* const FGenericCrashContext::EngineModeExUnknown = TEXT("Unset");
const TCHAR* const FGenericCrashContext::EngineModeExDirty = TEXT("Dirty");
const TCHAR* const FGenericCrashContext::EngineModeExVanilla = TEXT("Vanilla");

bool FGenericCrashContext::bIsInitialized = false;
uint32 FGenericCrashContext::OutOfProcessCrashReporterPid = 0;
volatile int64 FGenericCrashContext::OutOfProcessCrashReporterExitCode = 0;
int32 FGenericCrashContext::StaticCrashContextIndex = 0;

const FGuid FGenericCrashContext::ExecutionGuid = FGuid::NewGuid();

FEngineDataResetDelegate FGenericCrashContext::OnEngineDataReset;
FEngineDataSetDelegate FGenericCrashContext::OnEngineDataSet;

FGameDataResetDelegate FGenericCrashContext::OnGameDataReset;
FGameDataSetDelegate FGenericCrashContext::OnGameDataSet;

#if WITH_ADDITIONAL_CRASH_CONTEXTS
FAdditionalCrashContextDelegate FGenericCrashContext::AdditionalCrashContextDelegate;
FAdditionalCrashContextDelegateEx FGenericCrashContext::AdditionalCrashContextDelegateEx;
#endif //WITH_ADDITIONAL_CRASH_CONTEXTS

namespace NCached
{
	static FCriticalSection CriticalSection;
	
	static FSessionContext Session;
	static FUserSettingsContext UserSettings;
	static TArray<FString> EnabledPluginsList;
	static TMap<FString, FString> EngineData;
	static TMap<FString, FString> GameData;
	static TOptional<FGPUBreadcrumbCrashData> GPUBreadcrumbs;
	static TArray<FString> FilePaths;

	template <size_t CharCount, typename CharType>
	void Set(CharType(&Dest)[CharCount], const CharType* pSrc)
	{
		TCString<CharType>::Strncpy(Dest, pSrc, CharCount);
	}
}


FString FGenericCrashContext::GenerateNewCrashGuidBase()
{
	const FGuid Guid = FGuid::NewGuid();
	const FString IniPlatformName(FPlatformProperties::IniPlatformName());
	const FString GuidRoot = FString::Printf(TEXT("%s%s-%s"), CrashGUIDRootPrefix, *IniPlatformName, *Guid.ToString(EGuidFormats::Digits));
	NCached::Set(NCached::Session.CrashGUIDRoot, *GuidRoot);

	return GuidRoot;
}

void FGenericCrashContext::Initialize()
{
#if !NOINITCRASHREPORTER
	NCached::Session.bIsInternalBuild = FEngineBuildSettings::IsInternalBuild();
	NCached::Session.bIsPerforceBuild = FEngineBuildSettings::IsPerforceBuild();
	NCached::Session.bWithDebugInfo = FApp::GetIsWithDebugInfo();
	NCached::Session.bIsSourceDistribution = FEngineBuildSettings::IsSourceDistribution();
	NCached::Session.ProcessId = FPlatformProcess::GetCurrentProcessId();

	NCached::Set(NCached::Session.EngineVersion, *FEngineVersion::Current().ToString());
	NCached::Set(NCached::Session.EngineCompatibleVersion, *FEngineVersion::Current().ToString());
	NCached::Set(NCached::Session.BuildVersion, FApp::GetBuildVersion());
	NCached::Set(NCached::Session.GameName, *FString::Printf(TEXT("UE-%s"), FApp::GetProjectName()));
	NCached::Set(NCached::Session.GameSessionID, TEXT("")); // Updated by callback
	NCached::Set(NCached::Session.GameStateName, TEXT("")); // Updated by callback
	NCached::Set(NCached::Session.UserActivityHint, TEXT("")); // Updated by callback
	NCached::Set(NCached::Session.BuildConfigurationName, LexToString(FApp::GetBuildConfiguration()));
	NCached::Set(NCached::Session.ExecutableName, FPlatformProcess::ExecutableName());
	NCached::Set(NCached::Session.BaseDir, FPlatformProcess::BaseDir());
	NCached::Set(NCached::Session.RootDir, FPlatformMisc::RootDir());
	NCached::Set(NCached::Session.EpicAccountId, *FPlatformMisc::GetEpicAccountId());
	NCached::Set(NCached::Session.LoginIdStr, *FPlatformMisc::GetLoginId());

	// Unique string specifying the symbols to be used by CrashReporter
#ifdef UE_SYMBOLS_VERSION
	FString Symbols = FString(UE_SYMBOLS_VERSION);
#else
	FString Symbols = FString::Printf(TEXT("%s"), FApp::GetBuildVersion());
#endif
	Symbols = FString::Printf(TEXT("%s-%s-%s"), *Symbols, FPlatformMisc::GetUBTPlatform(), NCached::Session.BuildConfigurationName).Replace(TEXT("+"), TEXT("*"));
#ifdef UE_BUILD_FLAVOR
	const FString BuildFlavor(UE_BUILD_FLAVOR);
	Symbols = FString::Printf(TEXT("%s-%s"), *Symbols, *BuildFlavor);
#else
	const FString BuildFlavor;
#endif
	NCached::Set(NCached::Session.SymbolsLabel, *Symbols);
	if (Symbols.Len() >= UE_ARRAY_COUNT(NCached::Session.SymbolsLabel))
	{
		UE_LOG(LogInit, Error, TEXT("Symbols label too long (%d) for field size(%" SIZE_T_FMT "), truncated. This may cause problems with crash report symbolication."),
			Symbols.Len(), UE_ARRAY_COUNT(NCached::Session.SymbolsLabel));
	}

	FString OsVersion, OsSubVersion;
	FPlatformMisc::GetOSVersions(OsVersion, OsSubVersion);
	NCached::Set(NCached::Session.OsVersion, *OsVersion);
	NCached::Set(NCached::Session.OsSubVersion, *OsSubVersion);

	NCached::Session.NumberOfCores = FPlatformMisc::NumberOfCores();
	NCached::Session.NumberOfCoresIncludingHyperthreads = FPlatformMisc::NumberOfCoresIncludingHyperthreads();

	NCached::Set(NCached::Session.CPUVendor, *FPlatformMisc::GetCPUVendor());
	NCached::Set(NCached::Session.CPUBrand, *FPlatformMisc::GetCPUBrand());
	NCached::Session.CPUModelID = FPlatformMisc::GetCPUModelID();
	NCached::Session.CPUMicrocodeRevision = FPlatformMisc::GetCPUMicrocodeRevision();

	NCached::Set(NCached::Session.PrimaryGPUBrand, *FPlatformMisc::GetPrimaryGPUBrand());
	NCached::Set(NCached::Session.UserName, FPlatformProcess::UserName());
	NCached::Set(NCached::Session.DefaultLocale, *FPlatformMisc::GetDefaultLocale());

	NCached::Set(NCached::Session.PlatformName, ANSI_TO_TCHAR(FPlatformProperties::PlatformName()));
	NCached::Set(NCached::Session.PlatformNameIni, ANSI_TO_TCHAR(FPlatformProperties::IniPlatformName()));
	NCached::Set(NCached::Session.AttendedStatus, AttendedStatusToString(EUnattendedStatus::Unknown));

	// Information that cannot be gathered if command line is not initialized (e.g. crash during static init)
	if (FCommandLine::IsInitialized())
	{
		NCached::Session.bIsUERelease = FApp::IsEngineInstalled();
		NCached::Set(NCached::Session.CommandLine, (FCommandLine::IsInitialized() ? FCommandLine::GetOriginalForLogging() : TEXT("")));
		NCached::Set(NCached::Session.EngineMode, FGenericPlatformMisc::GetEngineMode());
		NCached::Set(NCached::Session.EngineModeEx, FGenericCrashContext::EngineModeExUnknown); // Updated from callback

		NCached::Set(NCached::UserSettings.LogFilePath, *FPlatformOutputDevices::GetAbsoluteLogFilename());

		// Use -epicapp value from the commandline to start. This will also be set by the game
		FParse::Value(FCommandLine::Get(), TEXT("EPICAPP="), NCached::Session.DeploymentName, CR_MAX_GENERIC_FIELD_CHARS, true);

		// Using the -fullcrashdump parameter will cause full memory minidumps to be created for crashes
		NCached::Session.CrashDumpMode = (int32)ECrashDumpMode::Default;
		if (FPlatformMisc::SupportsFullCrashDumps() && FCommandLine::IsInitialized())
		{
			const TCHAR* CmdLine = FCommandLine::Get();
			if (FParse::Param(CmdLine, TEXT("fullcrashdumpalways")))
			{
				NCached::Session.CrashDumpMode = (int32)ECrashDumpMode::FullDumpAlways;
			}
			else if (FParse::Param(CmdLine, TEXT("fullcrashdump")))
			{
				NCached::Session.CrashDumpMode = (int32)ECrashDumpMode::FullDump;
			}
		}

		const bool IsUnattended = FApp::IsUnattended();

		NCached::UserSettings.bNoDialog = IsUnattended|| IsRunningDedicatedServer();
		NCached::Set(NCached::Session.AttendedStatus, AttendedStatusToString(IsUnattended ? EUnattendedStatus::Unattended : EUnattendedStatus::Attended));
	}

	// Create a unique base guid for bug report ids
	const FString GuidRoot = GenerateNewCrashGuidBase();
	UE_LOGF(LogInit, Log, "Session CrashGUID >====================================================\n         Session CrashGUID >   %ls\n         Session CrashGUID >====================================================", *GuidRoot);
	if (FInternationalization::IsAvailable())
	{
		FInternationalization& Internationalization = FInternationalization::Get();
		NCached::Session.LanguageLCID = Internationalization.GetCurrentCulture()->GetLCID();

		Internationalization.OnCultureChanged().AddLambda([]()
		{
			NCached::Session.LanguageLCID = FInternationalization::Get().GetCurrentCulture()->GetLCID();
		});
	}

	// Initialize delegate for updating SecondsSinceStart, because FPlatformTime::Seconds() is not POSIX safe.
	const float PollingInterval = 1.0f;
	FTSTicker::GetCoreTicker().AddTicker( FTickerDelegate::CreateLambda( []( float DeltaTime )
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_NCachedCrashContextProperties_LambdaTicker);

		NCached::Session.SecondsSinceStart = int32(FPlatformTime::Seconds() - GStartTime);
		return true;
	} ), PollingInterval );

	FCoreDelegates::UserActivityStringChanged.AddLambda([](const FString& InUserActivity)
	{
		NCached::Set(NCached::Session.UserActivityHint, *InUserActivity);
	});

	FCoreDelegates::GameSessionIDChanged.AddLambda([](const FString& InGameSessionID)
	{
		NCached::Set(NCached::Session.GameSessionID, *InGameSessionID);
	});

	FCoreDelegates::GameStateClassChanged.AddLambda([](const FString& InGameStateName)
	{
		NCached::Set(NCached::Session.GameStateName, *InGameStateName);
	});

	FCoreDelegates::CrashOverrideParamsChanged.AddLambda([](const FCrashOverrideParameters& InParams)
	{
		if (InParams.bSetGameNameSuffix)
		{
			NCached::Set(NCached::Session.GameName, *(FString("UE-") + FApp::GetProjectName() + InParams.GameNameSuffix));
		}
		if (InParams.SendUnattendedBugReports.IsSet())
		{
			NCached::UserSettings.bSendUnattendedBugReports = InParams.SendUnattendedBugReports.GetValue();
		}
		if (InParams.SendUsageData.IsSet())
		{
			NCached::UserSettings.bSendUsageData = InParams.SendUsageData.GetValue();
		}
		SerializeTempCrashContextToFile();
	});

	FCoreDelegates::GetOnPostEngineInit().AddLambda([] 
	{
		// IsEditor global have been properly initialized here.
		NCached::Set(NCached::Session.EngineMode, FGenericPlatformMisc::GetEngineMode());
	});

	FCoreDelegates::IsVanillaProductChanged.AddLambda([](bool bIsVanilla)
	{
		NCached::Set(NCached::Session.EngineModeEx, bIsVanilla ? FGenericCrashContext::EngineModeExVanilla : FGenericCrashContext::EngineModeExDirty);
	});

	FCoreDelegates::TSConfigReadyForUse().AddStatic(FGenericCrashContext::InitializeFromConfig);

	FCoreDelegates::OnPostFork.AddLambda([](EForkProcessRole Role)
	{
		if (Role == EForkProcessRole::Child)
		{
			UE_LOGF(LogCrashContext, VeryVerbose, "Updating forked child Session ProcessID: %u -> %u", NCached::Session.ProcessId, FPlatformProcess::GetCurrentProcessId());

			NCached::Session.ProcessId = FPlatformProcess::GetCurrentProcessId();

			// Generate a new unique CrashGuidBase for this child
			const FString GuidRoot = GenerateNewCrashGuidBase();
			UE_LOGF(LogInit, Log, "Session CrashGUID >====================================================\n                                       Session CrashGUID >   %ls\n                                       Session CrashGUID >====================================================", *GuidRoot);			
			SerializeTempCrashContextToFile();
		}
	});

	SetEngineData(TEXT("BuildFlavor"), BuildFlavor);

	// Store some additional info in the generic maps if it is available
	if (const TCHAR* BuildURL = FApp::GetBuildURL())
	{
		SetEngineData(TEXT("BuildURL"), BuildURL);
	}
	if (const TCHAR* ExecutingJobURL = FApp::GetExecutingJobURL())
	{
		SetEngineData(TEXT("ExecutingJobURL"), ExecutingJobURL);
	}

	SerializeTempCrashContextToFile();

	CleanupPlatformSpecificFiles();

	bIsInitialized = true;
#endif	// !NOINITCRASHREPORTER
}

// When encoding the plugins list and engine and game data key/value pairs into the dynamic data segment
// we use 1 and 2 to denote delimiter and equals respectively. This is necessary since the values could 
// contain any characters normally used for delimiting.
const TCHAR* CR_PAIR_DELIM = TEXT("\x01");
const TCHAR* CR_PAIR_EQ = TEXT("\x02");

void FGenericCrashContext::InitializeFromContext(const FSessionContext& Session, const TCHAR* EnabledPluginsStr, const TCHAR* EngineDataStr, const TCHAR* GameDataStr, const TCHAR* FilePathData, const FGPUBreadcrumbsSharedContext* GPUBreadcrumbs)
{
	static const TCHAR* TokenDelim[] = { CR_PAIR_DELIM, CR_PAIR_EQ };

	// Copy the session struct which should be all pod types and fixed size buggers
	FMemory::Memcpy(NCached::Session, Session);
	
	// Parse the loaded plugins string, assume comma delimited values.
	if (EnabledPluginsStr)
	{
		TArray<FString> Tokens;
		FString(EnabledPluginsStr).ParseIntoArray(Tokens, TokenDelim, 2, true);

		for (FString& Token : Tokens)
		{
			if (Token.StartsWith(TEXT("{")) && Token.EndsWith(TEXT("}")))
			{
				NCached::EnabledPluginsList.Add(Token);
			}
		}
	}

	// Parse engine data, comma delimited key=value pairs.
	if (EngineDataStr)
	{
		TArray<FString> Tokens;
		FString(EngineDataStr).ParseIntoArray(Tokens, TokenDelim, 2, true);
		int32 i = 0;
		while ((i + 1) < Tokens.Num())
		{
			const FString& Key = Tokens[i++];
			const FString& Value = Tokens[i++];
			NCached::EngineData.Add(Key, Value);
		}
	}

	// Parse engine data, comma delimited key=value pairs.
	if (GameDataStr)
	{
		TArray<FString> Tokens;
		FString(GameDataStr).ParseIntoArray(Tokens, TokenDelim, 2, true);
		int32 i = 0;
		while ((i + 1) < Tokens.Num())
		{
			const FString& Key = Tokens[i++];
			const FString& Value = Tokens[i++];
			NCached::GameData.Add(Key, Value);
		}
	}

	if (FilePathData)
	{
		TArray<FString> Tokens;
		FString(FilePathData).ParseIntoArray(Tokens, TokenDelim, 2, true);

		for (FString& Token : Tokens)
		{
			NCached::FilePaths.Add(Token);
		}
	}

	if (GPUBreadcrumbs && GPUBreadcrumbs->NumQueues > 0)
	{
		SetGPUBreadcrumbs(GPUBreadcrumbsFromSharedContext(*GPUBreadcrumbs));
	}

	SerializeTempCrashContextToFile();

	bIsInitialized = true;
}

const FSessionContext& FGenericCrashContext::GetCachedSessionContext()
{
	return NCached::Session;
}

FString FGenericCrashContext::GetGameName()
{
	return FString::Printf(TEXT("UE-%s"), FApp::GetProjectName());
}


void FGenericCrashContext::CopySharedCrashContext(FSharedCrashContext& Dst)
{
	//Copy the session
	FMemory::Memcpy(Dst.SessionContext, NCached::Session);
	FMemory::Memcpy(Dst.UserSettings, NCached::UserSettings);

	TStringBuilder<0> DynamicDataBuilder;
	DynamicDataBuilder.Reserve(CR_MAX_DYNAMIC_BUFFER_CHARS);

	Dst.EnabledPluginsOffset = (uint32)DynamicDataBuilder.Len();
#if UE_CRASH_REPORTER_WITH_LOADED_PLUGINS
	Dst.EnabledPluginsNum = NCached::EnabledPluginsList.Num();
	for (const FString& Plugin : NCached::EnabledPluginsList)
	{
		DynamicDataBuilder.Append(Plugin);
		DynamicDataBuilder.Append(CR_PAIR_DELIM);
	}
#else
	Dst.EnabledPluginsNum = 0;
#endif

	Dst.EngineDataOffset = (uint32)DynamicDataBuilder.Len();
	Dst.EngineDataNum = NCached::EngineData.Num();
	for (const TPair<FString, FString>& Pair : NCached::EngineData)
	{
		DynamicDataBuilder.Append(Pair.Key);
		DynamicDataBuilder.Append(CR_PAIR_EQ);
		DynamicDataBuilder.Append(Pair.Value);
		DynamicDataBuilder.Append(CR_PAIR_DELIM);
	}

	Dst.GameDataOffset = (uint32)DynamicDataBuilder.Len();
	Dst.GameDataNum = NCached::GameData.Num();
	for (const TPair<FString, FString>& Pair : NCached::GameData)
	{
		DynamicDataBuilder.Append(Pair.Key);
		DynamicDataBuilder.Append(CR_PAIR_EQ);
		DynamicDataBuilder.Append(Pair.Value);
		DynamicDataBuilder.Append(CR_PAIR_DELIM);
	}

	Dst.FilePathsOffset = (uint32)DynamicDataBuilder.Len();
	Dst.FilePathsNum = NCached::FilePaths.Num();
	for (const FString& FilePath : NCached::FilePaths)
	{
		DynamicDataBuilder.Append(FilePath);
		DynamicDataBuilder.Append(CR_PAIR_DELIM);
	}

	FCString::Strncpy(Dst.DynamicData, DynamicDataBuilder.ToString(), CR_MAX_DYNAMIC_BUFFER_CHARS - 1);
	Dst.DynamicData[CR_MAX_DYNAMIC_BUFFER_CHARS - 1] = 0;

	// Copy GPU breadcrumbs.
	if (NCached::GPUBreadcrumbs.IsSet())
	{
		GPUBreadcrumbsToSharedContext(*NCached::GPUBreadcrumbs, Dst.GPUBreadcrumbs);
	}
}

void FGenericCrashContext::SetMemoryStats(const FPlatformMemoryStats& InMemoryStats)
{
	NCached::Session.MemoryStats = InMemoryStats;

	// Update cached OOM stats
	NCached::Session.bIsOOM = FPlatformMemory::bIsOOM;
	NCached::Session.OOMAllocationSize = FPlatformMemory::OOMAllocationSize;
	NCached::Session.OOMAllocationAlignment = FPlatformMemory::OOMAllocationAlignment;

	SerializeTempCrashContextToFile();
}

void FGenericCrashContext::InitializeFromConfig()
{
#if !NOINITCRASHREPORTER
	PurgeOldCrashConfig();

	const bool bForceGetSection = false;
	FConfigFile CrashConfigFile;

	if (const FConfigSection* CRCConfigSection = GConfig->GetSection(ConfigSectionName, bForceGetSection, GEngineIni))
	{
		// Create a config file and save to a temp location. This file will be copied to
		// the crash folder for all crash reports create by this session.

		FConfigSection CRCConfigSectionCopy(*CRCConfigSection);
		CrashConfigFile.Add(ConfigSectionName, CRCConfigSectionCopy);

		CrashConfigFile.Dirty = true;
	}

#if WITH_EDITOR
	if (const FConfigSection* CRCEConfigSection = GConfig->GetSection(ConfigSectionName, bForceGetSection, GEditorIni))
	{
		for (FConfigSection::TConstIterator ConfigValueIt(*CRCEConfigSection); ConfigValueIt; ++ConfigValueIt)
		{
			CrashConfigFile.SetInSection(ConfigSectionName, ConfigValueIt.Key(), ConfigValueIt.Value().GetValue());
		}

		CrashConfigFile.Dirty = true;
	}
#endif

	if (CrashConfigFile.Dirty)
	{
		CrashConfigFile.Write(FString(GetCrashConfigFilePath()));
	}

	// Read the initial un-localized crash context text
	UpdateLocalizedStrings();

#if WITH_EDITOR
	// Set privacy settings -> WARNING: Ensure those setting have a default values in Engine/Config/BaseEditorSettings.ini file, otherwise, they will not be found.
	GConfig->GetBool(TEXT("/Script/UnrealEd.CrashReportsPrivacySettings"), TEXT("bSendUnattendedBugReports"), NCached::UserSettings.bSendUnattendedBugReports, GEditorSettingsIni);
	GConfig->GetBool(TEXT("/Script/UnrealEd.AnalyticsPrivacySettings"), TEXT("bSendUsageData"), NCached::UserSettings.bSendUsageData, GEditorSettingsIni);
#elif CRASH_REPORTER_WITH_ANALYTICS
	NCached::UserSettings.bSendUnattendedBugReports = true; // Give CRC permission to generate and send an 'AbnormalShutdown' report if the application died suddently, mainly to collect the logs post-mortem and count those occurrences.
	NCached::UserSettings.bSendUsageData = true; // Give CRC permission to send its analytics and the application session summary (if one exist)
#endif
	
	// Write a marker file to disk indicating the user has allowed unattended crash reports being
	// sent. This allows us to submit reports for crashes during static initialization when user
	// settings are not available. 
	FString MarkerFilePath = FPaths::Combine(FPlatformProcess::ApplicationSettingsDir(), TEXT("NotAllowedUnattendedBugReports"));
	if (!NCached::UserSettings.bSendUnattendedBugReports)
	{
		TUniquePtr<IFileHandle> File(FPlatformFileManager::Get().GetPlatformFile().OpenWrite(*MarkerFilePath));
	}

	// Make sure we get updated text once the localized version is loaded
	FTextLocalizationManager::Get().OnTextRevisionChangedEvent.AddStatic(&UpdateLocalizedStrings);

	SerializeTempCrashContextToFile();

#endif	// !NOINITCRASHREPORTER
}

void FGenericCrashContext::UpdateLocalizedStrings()
{
#if !NOINITCRASHREPORTER
	// Allow overriding the crash text
	FText CrashReportClientRichText;
	if (GConfig->GetText(TEXT("CrashContextProperties"), TEXT("CrashReportClientRichText"), CrashReportClientRichText, GEngineIni))
	{
		NCached::Set(NCached::Session.CrashReportClientRichText, *CrashReportClientRichText.ToString());
	}
#endif
}

void FGenericCrashContext::SetAnticheatProvider(const FString& AnticheatProvider)
{
	NCached::Set(NCached::Session.AnticheatProvider, *AnticheatProvider);

	SerializeTempCrashContextToFile();
}

void FGenericCrashContext::OnThreadStuck(uint32 ThreadId)
{
	if (!NCached::Session.bIsStuck || NCached::Session.StuckThreadId != ThreadId)
	{
		NCached::Session.bIsStuck = true;
		NCached::Session.StuckThreadId = ThreadId;

		SerializeTempCrashContextToFile();
	}
}

void FGenericCrashContext::OnThreadUnstuck(uint32 ThreadId)
{
	if (NCached::Session.bIsStuck)
	{
		NCached::Session.bIsStuck = false;
		NCached::Session.StuckThreadId = 0;

		SerializeTempCrashContextToFile();
	}
}

FGenericCrashContext::FGenericCrashContext(ECrashContextType InType, const TCHAR* InErrorMessage)
	: Type(InType)
	, CrashedThreadId(~uint32(0))
	, ErrorMessage(InErrorMessage)
	, NumMinidumpFramesToIgnore(0)
{
	CommonBuffer.Reserve( 128 * 1024 ); // NOTE: The Editor command 'debug crash' uses about 112K characters on Windows when the crash is serialized by SerializeContentToBuffer()
	CrashContextIndex = StaticCrashContextIndex++;
}

FString FGenericCrashContext::GetTempSessionContextFilePath(uint64 ProcessID)
{
	return FPlatformProcess::UserTempDir() / FString::Printf(TEXT("UECrashContext-%" UINT64_FMT ".xml"), ProcessID);
}

void FGenericCrashContext::CleanupTempSessionContextFiles(const FTimespan& ExpirationAge)
{
	const FString BasePathname = FPlatformProcess::UserTempDir() / FString::Printf(TEXT("UECrashContext-"));
	IFileManager::Get().IterateDirectory(FPlatformProcess::UserTempDir(), [&BasePathname, &ExpirationAge](const TCHAR* Pathname, bool bIsDirectory) -> bool
	{
		if (bIsDirectory)
		{
			return true; // Looking for files, continue.
		}

		FStringView PathnameView(Pathname);
		if (PathnameView.EndsWith(TEXT(".xml")) && PathnameView.StartsWith(BasePathname))
		{
			if (IFileManager::Get().GetFileAgeSeconds(Pathname) > ExpirationAge.GetTotalSeconds())
			{
				// Extract the process ID from the pathname view.
				static_assert(sizeof(decltype(FPlatformProcess::GetCurrentProcessId())) == 4, "The code below assumes the process ID is 4 bytes");
				constexpr uint32 MaxPidStrLen = 10; // std::numeric_limit<uint32>::max() is 4294967295 -> 10 characters.
				TCHAR PidStr[MaxPidStrLen + 1]; // +1 for null terminator.
				int32 CharIndex = 0;
				PathnameView.FindLastChar(TEXT('-'), CharIndex);
				PathnameView.RemovePrefix(CharIndex + 1); // Remove everything up to the last '-'.
				PathnameView.FindLastChar(TEXT('.'), CharIndex);
				PathnameView.RemoveSuffix(PathnameView.Len() - CharIndex); // Remove the trailing '.xml'

				if (PathnameView.Len() <= MaxPidStrLen)
				{
					// Put back what we expect to be the PID into a null terminated string.
					PathnameView.CopyString(PidStr, PathnameView.Len());
					PidStr[PathnameView.Len()] = TEXT('\0');

					// Converts the PID string, validating it only contained digits.
					TCHAR* End = nullptr;
					int64 ProcessId = FCString::Strtoi64(PidStr, &End, 10);
					if (End == PidStr + PathnameView.Len())
					{
						// Ensure the process that created this context file is not running anymore before deleting it.
						if (!FPlatformProcess::IsApplicationRunning(static_cast<uint32>(ProcessId)))
						{
							IFileManager::Get().Delete(Pathname);
						}
					}
				}
			}
		}

		return true; // Iterate to next file.
	});
}

TOptional<int32> FGenericCrashContext::GetOutOfProcessCrashReporterExitCode()
{
	TOptional<int32> ExitCode;
	int64 ExitCodeData = FPlatformAtomics::AtomicRead(&OutOfProcessCrashReporterExitCode);
	if (ExitCodeData & 0xFFFFFFFF00000000) // If one bit in the 32 MSB is set, the out of process exit code is set in the 32 LSB.
	{
		ExitCode.Emplace(static_cast<int32>(ExitCodeData)); // Truncate the 32 MSB.
	}
	return ExitCode;
}

void FGenericCrashContext::SetOutOfProcessCrashReporterExitCode(int32 ExitCode)
{
	int64 ExitCodeData = (1ll << 32) | ExitCode; // Set a bit in the 32 MSB to signal that the exit code is set
	FPlatformAtomics::AtomicStore(&OutOfProcessCrashReporterExitCode, ExitCodeData);
}

void FGenericCrashContext::SerializeTempCrashContextToFile()
{
	if (!IsOutOfProcessCrashReporter())
	{
		return;
	}

	FString SessionBuffer;
	SessionBuffer.Reserve(32 * 1024);

	AddHeader(SessionBuffer);

	SerializeSessionContext(SessionBuffer);
	SerializeUserSettings(SessionBuffer);
	
	// Add the engine data
	BeginSection(SessionBuffer, EngineDataTag);
	for (const TPair<FString, FString>& Pair : NCached::EngineData)
	{
		AddCrashPropertyInternal(SessionBuffer, *Pair.Key, *Pair.Value);
	}
	EndSection(SessionBuffer, EngineDataTag);

	// Add the game data
	BeginSection(SessionBuffer, GameDataTag);
	for (const TPair<FString, FString>& Pair : NCached::GameData)
	{
		AddCrashPropertyInternal(SessionBuffer, *Pair.Key, *Pair.Value);
	}
	EndSection(SessionBuffer, GameDataTag);

	AddFooter(SessionBuffer);

	const FString SessionFilePath = GetTempSessionContextFilePath(NCached::Session.ProcessId);
	FFileHelper::SaveStringToFile(SessionBuffer, *SessionFilePath);
}

// This function may be called in the crashing executable or in an external crash reporter program. Take care with accessing global variables vs member variables or 
// fields in NCached!
void FGenericCrashContext::SerializeSessionContext(FString& Buffer)
{
	AddCrashPropertyInternal(Buffer, FGenericCrashContext::ProcessIdTag, NCached::Session.ProcessId);
	AddCrashPropertyInternal(Buffer, FGenericCrashContext::SecondsSinceStartTag, NCached::Session.SecondsSinceStart);

	AddCrashPropertyInternal(Buffer, FGenericCrashContext::IsInternalBuildTag, NCached::Session.bIsInternalBuild);
	AddCrashPropertyInternal(Buffer, FGenericCrashContext::IsPerforceBuildTag, NCached::Session.bIsPerforceBuild);
	AddCrashPropertyInternal(Buffer, FGenericCrashContext::IsWithDebugInfoTag, NCached::Session.bWithDebugInfo);
	AddCrashPropertyInternal(Buffer, FGenericCrashContext::IsSourceDistributionTag, NCached::Session.bIsSourceDistribution);

	if (FCString::Strlen(NCached::Session.GameName) > 0)
	{
		AddCrashPropertyInternal(Buffer, FGenericCrashContext::GameNameTag, NCached::Session.GameName);
	}
	else
	{
		const TCHAR* ProjectName = FApp::GetProjectName();
		if (ProjectName != nullptr && ProjectName[0] != 0)
		{
			AddCrashPropertyInternal(Buffer, FGenericCrashContext::GameNameTag, *FString::Printf(TEXT("UE-%s"), ProjectName));
		}
		else
		{
			AddCrashPropertyInternal(Buffer, FGenericCrashContext::GameNameTag, TEXT(""));
		}
	}
	AddCrashPropertyInternal(Buffer, TEXT("ExecutableName"), NCached::Session.ExecutableName);
	AddCrashPropertyInternal(Buffer, TEXT("BuildConfiguration"), NCached::Session.BuildConfigurationName);
	AddCrashPropertyInternal(Buffer, TEXT("GameSessionID"), NCached::Session.GameSessionID);

	AddCrashPropertyInternal(Buffer, TEXT("PlatformName"), NCached::Session.PlatformName);
	AddCrashPropertyInternal(Buffer, TEXT("PlatformFullName"), NCached::Session.PlatformName);
	AddCrashPropertyInternal(Buffer, TEXT("PlatformNameIni"), NCached::Session.PlatformNameIni);
	AddCrashPropertyInternal(Buffer, TEXT("EngineMode"), NCached::Session.EngineMode);
	AddCrashPropertyInternal(Buffer, TEXT("EngineModeEx"), NCached::Session.EngineModeEx);

	AddCrashPropertyInternal(Buffer, FGenericCrashContext::DeploymentNameTag, NCached::Session.DeploymentName);

	AddCrashPropertyInternal(Buffer, TEXT("EngineVersion"), NCached::Session.EngineVersion); 
	AddCrashPropertyInternal(Buffer, TEXT("EngineCompatibleVersion"), NCached::Session.EngineCompatibleVersion); 
	AddCrashPropertyInternal(Buffer, TEXT("CommandLine"), NCached::Session.CommandLine);
	AddCrashPropertyInternal(Buffer, TEXT("LanguageLCID"), NCached::Session.LanguageLCID);
	AddCrashPropertyInternal(Buffer, TEXT("AppDefaultLocale"), NCached::Session.DefaultLocale);
	AddCrashPropertyInternal(Buffer, BuildVersionTag, NCached::Session.BuildVersion);
	AddCrashPropertyInternal(Buffer, TEXT("Symbols"), NCached::Session.SymbolsLabel);
	AddCrashPropertyInternal(Buffer, TEXT("IsUERelease"), NCached::Session.bIsUERelease);

	// Need to set this at the time of the crash to check if requesting exit had been called
	AddCrashPropertyInternal(Buffer, FGenericCrashContext::IsRequestingExitTag, NCached::Session.bIsExitRequested);

	// Remove periods from user names to match AutoReporter user names
	// The name prefix is read by CrashRepository.AddNewCrash in the website code
	const bool bSendUserName = NCached::Session.bIsInternalBuild;
	FString SanitizedUserName = FString(NCached::Session.UserName).Replace(TEXT("."), TEXT(""));
	AddCrashPropertyInternal(Buffer, TEXT("UserName"), bSendUserName ? *SanitizedUserName : TEXT(""));

	AddCrashPropertyInternal(Buffer, TEXT("BaseDir"), NCached::Session.BaseDir);
	AddCrashPropertyInternal(Buffer, TEXT("RootDir"), NCached::Session.RootDir);
	AddCrashPropertyInternal(Buffer, TEXT("MachineId"), *FString(NCached::Session.LoginIdStr).ToUpper());
	AddCrashPropertyInternal(Buffer, TEXT("LoginId"), NCached::Session.LoginIdStr);
	AddCrashPropertyInternal(Buffer, TEXT("EpicAccountId"), NCached::Session.EpicAccountId);

	AddCrashPropertyInternal(Buffer, TEXT("SourceContext"), TEXT(""));
	AddCrashPropertyInternal(Buffer, TEXT("UserDescription"), TEXT(""));
	AddCrashPropertyInternal(Buffer, TEXT("UserActivityHint"), NCached::Session.UserActivityHint);
	AddCrashPropertyInternal(Buffer, TEXT("CrashDumpMode"), NCached::Session.CrashDumpMode);
	AddCrashPropertyInternal(Buffer, TEXT("GameStateName"), NCached::Session.GameStateName);

	// Add misc stats.
	AddCrashPropertyInternal(Buffer, TEXT("Misc.NumberOfCores"), NCached::Session.NumberOfCores);
	AddCrashPropertyInternal(Buffer, TEXT("Misc.NumberOfCoresIncludingHyperthreads"), NCached::Session.NumberOfCoresIncludingHyperthreads);
	AddCrashPropertyInternal(Buffer, TEXT("Misc.Is64bitOperatingSystem"), 1);

	AddCrashPropertyInternal(Buffer, TEXT("Misc.CPUVendor"), NCached::Session.CPUVendor);
	AddCrashPropertyInternal(Buffer, TEXT("Misc.CPUBrand"), NCached::Session.CPUBrand);
	AddCrashPropertyInternal(Buffer, TEXT("Misc.CPUModelID"), NCached::Session.CPUModelID);
	AddCrashPropertyInternal(Buffer, TEXT("Misc.CPUMicrocodeRevision"), NCached::Session.CPUMicrocodeRevision);
	AddCrashPropertyInternal(Buffer, TEXT("Misc.PrimaryGPUBrand"), NCached::Session.PrimaryGPUBrand);
	AddCrashPropertyInternal(Buffer, TEXT("Misc.OSVersionMajor"), NCached::Session.OsVersion);
	AddCrashPropertyInternal(Buffer, TEXT("Misc.OSVersionMinor"), NCached::Session.OsSubVersion);
	AddCrashPropertyInternal(Buffer, TEXT("Misc.AnticheatProvider"), NCached::Session.AnticheatProvider);
	if (NCached::Session.bIsStuck)
	{
		AddCrashPropertyInternal(Buffer, TEXT("Misc.IsStuck"), NCached::Session.bIsStuck);
		AddCrashPropertyInternal(Buffer, TEXT("Misc.StuckThreadId"), NCached::Session.StuckThreadId);
	}

	// FPlatformMemory::GetConstants is called in the GCreateMalloc, so we can assume it is always valid.
	{
		// Add memory stats.
		const FPlatformMemoryConstants& MemConstants = FPlatformMemory::GetConstants();

		AddCrashPropertyInternal(Buffer, TEXT("MemoryStats.TotalPhysical"), (uint64) MemConstants.TotalPhysical);
		AddCrashPropertyInternal(Buffer, TEXT("MemoryStats.TotalVirtual"), (uint64) MemConstants.TotalVirtual);
		AddCrashPropertyInternal(Buffer, TEXT("MemoryStats.PageSize"), (uint64) MemConstants.PageSize);
		AddCrashPropertyInternal(Buffer, TEXT("MemoryStats.TotalPhysicalGB"), MemConstants.TotalPhysicalGB);
	}

	AddCrashPropertyInternal(Buffer, TEXT("MemoryStats.AvailablePhysical"), (uint64) NCached::Session.MemoryStats.AvailablePhysical);
	AddCrashPropertyInternal(Buffer, TEXT("MemoryStats.AvailableVirtual"), (uint64) NCached::Session.MemoryStats.AvailableVirtual);
	AddCrashPropertyInternal(Buffer, TEXT("MemoryStats.UsedPhysical"), (uint64) NCached::Session.MemoryStats.UsedPhysical);
	AddCrashPropertyInternal(Buffer, TEXT("MemoryStats.PeakUsedPhysical"), (uint64) NCached::Session.MemoryStats.PeakUsedPhysical);
	AddCrashPropertyInternal(Buffer, TEXT("MemoryStats.UsedVirtual"), (uint64) NCached::Session.MemoryStats.UsedVirtual);
	AddCrashPropertyInternal(Buffer, TEXT("MemoryStats.PeakUsedVirtual"), (uint64) NCached::Session.MemoryStats.PeakUsedVirtual);
	AddCrashPropertyInternal(Buffer, TEXT("MemoryStats.bIsOOM"), (int) NCached::Session.bIsOOM);
	AddCrashPropertyInternal(Buffer, TEXT("MemoryStats.OOMAllocationSize"), NCached::Session.OOMAllocationSize);
	AddCrashPropertyInternal(Buffer, TEXT("MemoryStats.OOMAllocationAlignment"), NCached::Session.OOMAllocationAlignment);
}

void FGenericCrashContext::SerializeUserSettings(FString& Buffer)
{
	AddCrashPropertyInternal(Buffer, TEXT("NoDialog"), NCached::UserSettings.bNoDialog);
	AddCrashPropertyInternal(Buffer, TEXT("SendUnattendedBugReports"), NCached::UserSettings.bSendUnattendedBugReports);
	AddCrashPropertyInternal(Buffer, TEXT("SendUsageData"), NCached::UserSettings.bSendUsageData);
	AddCrashPropertyInternal(Buffer, FGenericCrashContext::LogFilePathTag, FPlatformOutputDevices::GetAbsoluteLogFilename()); // Don't use the value cached, it may be out of date.
}

// This function may be called in the crashing executable or in an external crash reporter program. Take care with accessing global variables vs member variables or 
// fields in NCached!
void FGenericCrashContext::SerializeContentToBuffer() const
{
	// Clear the buffer in case the content is serialized more than once, keeping the most up to date values and preventing to store several XML documents in the same buffer.
	// When the buffer is passed to our XML reader, only the first document <FGenericCrashContext></<FGenericCrashContext> is read and further ones (most recent ones) are ignored.
	CommonBuffer.Reset();

	TCHAR CrashGUID[CrashGUIDLength];
	GetUniqueCrashName(CrashGUID, CrashGUIDLength);

	// Must conform against:
	// https://www.securecoding.cert.org/confluence/display/seccode/SIG30-C.+Call+only+asynchronous-safe+functions+within+signal+handlers
	AddHeader(CommonBuffer);

	BeginSection( CommonBuffer, RuntimePropertiesTag );
	AddCrashProperty( CrashVersionTag, (int32)ECrashDescVersions::VER_3_CrashContext );
	AddCrashProperty( ExecutionGuidTag, *ExecutionGuid.ToString() );
	AddCrashProperty( CrashGuidTag, (const TCHAR*)CrashGUID);

	AddCrashProperty( IsEnsureTag, (Type == ECrashContextType::Ensure) );
	AddCrashProperty( IsStallTag, (Type == ECrashContextType::Stall) );
	AddCrashProperty( IsAssertTag, (Type == ECrashContextType::Assert) );
	AddCrashProperty( CrashTypeTag, GetCrashTypeString(Type) );
	AddCrashProperty( ErrorMessageTag, ErrorMessage );
	AddCrashProperty( CrashReporterMessageTag, NCached::Session.CrashReportClientRichText );
	AddCrashProperty( AttendedStatusTag, NCached::Session.AttendedStatus);

	SerializeSessionContext(CommonBuffer);

	// Legacy callstack element for current crash reporter
	AddCrashProperty( TEXT( "NumMinidumpFramesToIgnore"), NumMinidumpFramesToIgnore );
	// Allow platforms to override callstack property, on some platforms the callstack is not captured by native code, those callstacks can be substituted by platform code here.
	{
		CommonBuffer += TEXT("<CallStack>");
		CommonBuffer += GetCallstackProperty();
		CommonBuffer += TEXT("</CallStack>");
		CommonBuffer += LINE_TERMINATOR;
	}

	// Add new portable callstack element with crash stack
	AddPortableCallStack();
	AddPortableCallStackHash();

	AddGPUBreadcrumbs();

	{
		FString AllThreadStacks;
		if (GetPlatformAllThreadContextsString(AllThreadStacks))
		{
			CommonBuffer += TEXT("<Threads>");
			CommonBuffer += AllThreadStacks;
			CommonBuffer += TEXT("</Threads>");
			CommonBuffer += LINE_TERMINATOR;
		}
	}

	EndSection( CommonBuffer, RuntimePropertiesTag );

	// Add platform specific properties.
	BeginSection( CommonBuffer, PlatformPropertiesTag );
	AddPlatformSpecificProperties();
	// The name here is a bit cryptic, but we keep it to avoid breaking backend stuff.
	AddCrashProperty(TEXT("PlatformCallbackResult"), NCached::Session.CrashTrigger);
	// New name we can phase in for the crash trigger to distinguish real crashes from debug
	AddCrashProperty(TEXT("CrashTrigger"), NCached::Session.CrashTrigger);
	EndSection( CommonBuffer, PlatformPropertiesTag );

	// Add the engine data
	BeginSection( CommonBuffer, EngineDataTag );
	for (const TPair<FString, FString>& Pair : NCached::EngineData)
	{
		AddCrashProperty( *Pair.Key, *Pair.Value);
	}
	EndSection( CommonBuffer, EngineDataTag );

	// Add the game data
	BeginSection( CommonBuffer, GameDataTag );
	for (const TPair<FString, FString>& Pair : NCached::GameData)
	{
		AddCrashProperty( *Pair.Key, *Pair.Value);
	}
	EndSection( CommonBuffer, GameDataTag );

	// Writing out the list of plugin JSON descriptors causes us to run out of memory
	// in GMallocCrash on console, so enable this only for desktop platforms.
#if PLATFORM_DESKTOP
	if (NCached::EnabledPluginsList.Num() > 0)
	{
		BeginSection(CommonBuffer, EnabledPluginsTag);

		for (const FString& Str : NCached::EnabledPluginsList)
		{
			AddCrashProperty(TEXT("Plugin"), *Str);
		}

		EndSection(CommonBuffer, EnabledPluginsTag);
	}
#endif // PLATFORM_DESKTOP

	AddFooter(CommonBuffer);
}

const TCHAR* FGenericCrashContext::GetCallstackProperty() const
{
	return TEXT("");
}

void FGenericCrashContext::SetEngineExit(bool bIsExiting)
{
	NCached::Session.bIsExitRequested = IsEngineExitRequested();
}

void FGenericCrashContext::SetNumMinidumpFramesToIgnore(int InNumMinidumpFramesToIgnore)
{
	NumMinidumpFramesToIgnore = InNumMinidumpFramesToIgnore;
}

void FGenericCrashContext::SetDeploymentName(const FString& EpicApp)
{
	NCached::Set(NCached::Session.DeploymentName, *EpicApp);
}

const TCHAR* FGenericCrashContext::GetDeploymentName()
{
	return NCached::Session.DeploymentName;
}

void FGenericCrashContext::SetCrashTrigger(ECrashTrigger Type)
{
	NCached::Session.CrashTrigger = (int32)Type;
}

void FGenericCrashContext::GetUniqueCrashName(TCHAR* GUIDBuffer, int32 BufferSize) const
{
	FCString::Snprintf(GUIDBuffer, BufferSize, TEXT("%s_%04i"), NCached::Session.CrashGUIDRoot, CrashContextIndex);
}

const bool FGenericCrashContext::IsFullCrashDump() const
{
	if (FGenericCrashContext::IsTypeContinuable(Type))
	{
		return (NCached::Session.CrashDumpMode == (int32)ECrashDumpMode::FullDumpAlways);
	}
	else
	{
		return (NCached::Session.CrashDumpMode == (int32)ECrashDumpMode::FullDump) ||
			(NCached::Session.CrashDumpMode == (int32)ECrashDumpMode::FullDumpAlways);
	}
}

void FGenericCrashContext::SerializeAsXML( const TCHAR* Filename ) const
{
	SerializeContentToBuffer();
	// Use OS build-in functionality instead.
	FFileHelper::SaveStringToFile( CommonBuffer, Filename, FFileHelper::EEncodingOptions::AutoDetect );
}

void FGenericCrashContext::AddCrashPropertyInternal(FString& Buffer, FStringView PropertyName, FStringView PropertyValue)
{
	Buffer.Appendf(TEXT("<%.*s>%s</%.*s>" LINE_TERMINATOR_ANSI), 
		PropertyName.Len(), PropertyName.GetData(),
		*FXmlEscapedString(PropertyValue), 
		PropertyName.Len(), PropertyName.GetData()
		);
}

void FGenericCrashContext::AddPlatformSpecificProperties() const
{
	// Nothing really to do here. Can be overridden by the platform code.
	// @see FWindowsPlatformCrashContext::AddPlatformSpecificProperties
}

bool FGenericCrashContext::ShouldIncludeInPortableCallStackHash(const FCrashStackFrame& Frame) const
{
	// This may allocate if its the first time calling into this function
	static const TCHAR* ExeName = FPlatformProcess::ExecutableName();

	// We dont want this to be thrown into an FString as it will alloc memory
	const TCHAR* UEEditorName = TEXT("UnrealEditor");

	// If we are our own module or our module contains UnrealEditor we assume we own these. We cannot depend on offsets of system libs
	// as they may have different versions
	return Frame.ModuleName == ExeName || Frame.ModuleName.Contains(UEEditorName);
}

void FGenericCrashContext::AddPortableCallStackHash() const
{
	if (CallStack.Num() == 0)
	{
		AddCrashProperty(TEXT("PCallStackHash"), TEXT(""));
		return;
	}

	FSHA1 Sha;
	FSHAHash Hash;

	for (TArray<FCrashStackFrame>::TConstIterator It(CallStack); It; ++It)
	{
		if (ShouldIncludeInPortableCallStackHash(*It))
		{
			Sha.Update(reinterpret_cast<const uint8*>(&It->Offset), sizeof(It->Offset));
		}
	}

	Sha.Final();
	Sha.GetHash(Hash.Hash);

	FString EscapedPortableHash;

	// Allocations here on both the ToString and AppendEscapedXMLString it self adds to the out FString
	AppendEscapedXMLString(EscapedPortableHash, *Hash.ToString());

	AddCrashProperty(TEXT("PCallStackHash"), *EscapedPortableHash);
}

void FGenericCrashContext::AppendPortableCallstack(FString& OutBuffer, TConstArrayView<FCrashStackFrame> StackFrames)
{
	OutBuffer += LINE_TERMINATOR;

	// Get the max module name length for padding
	int32 MaxModuleLength = 0;
	for (const FCrashStackFrame& Frame : StackFrames)
	{
		MaxModuleLength = FMath::Max(MaxModuleLength, FXmlEscapedString(Frame.ModuleName).Len());
	}

	for (const FCrashStackFrame& Frame : StackFrames)
	{
		OutBuffer.Appendf(TEXT("%-*s 0x%016llx + %-16llx" LINE_TERMINATOR_ANSI), MaxModuleLength + 1, *FXmlEscapedString(Frame.ModuleName), Frame.BaseAddress, Frame.Offset);
	}
}

void FGenericCrashContext::AddPortableCallStack() const
{	
	if (CallStack.Num() == 0)
	{
		AddCrashProperty(PortableCallStackTag, TEXT(""));
		return;
	}

	BeginSection(CommonBuffer, PortableCallStackTag);
	AppendPortableCallstack(CommonBuffer, CallStack);
	EndSection(CommonBuffer, PortableCallStackTag);
}

void FGenericCrashContext::AddGPUBreadcrumbs() const
{
	if (!NCached::GPUBreadcrumbs.IsSet())
	{
		return;
	}
	
	BeginSection(CommonBuffer, TEXT("GPUBreadcrumbs"));

	// We use a version indicator for the format used by the breadcrumbs
	// string, so that parsers can know what to expect and don't break
	// if changes are made in the format exported by the engine.
	AddCrashProperty(TEXT("FormatVersion"), NCached::GPUBreadcrumbs->Version);
	AddCrashProperty(TEXT("Source"), NCached::GPUBreadcrumbs->SourceName);

	for (auto& [Queue, Breadcrumbs] : NCached::GPUBreadcrumbs->Queues)
	{
		BeginSection(CommonBuffer, TEXT("Queue"));

		AddCrashProperty(TEXT("Name"), Queue);
		AddCrashProperty(TEXT("FullHash"), Breadcrumbs.FullHash.ToString());
		AddCrashProperty(TEXT("ActiveHash"), Breadcrumbs.ActiveHash.ToString());
		AddCrashProperty(TEXT("Breadcrumbs"), Breadcrumbs.BreadcrumbString);

		EndSection(CommonBuffer, TEXT("Queue"));
	}
	EndSection(CommonBuffer, TEXT("GPUBreadcrumbs"));
}

void FGenericCrashContext::AddHeader(FString& Buffer)
{
	Buffer += TEXT("<?xml version=\"1.0\" encoding=\"UTF-8\"?>" LINE_TERMINATOR_ANSI);
	BeginSection(Buffer, TEXT("FGenericCrashContext") );
}

void FGenericCrashContext::AddFooter(FString& Buffer)
{
	EndSection(Buffer, TEXT( "FGenericCrashContext" ));
}

void FGenericCrashContext::BeginSection(FString& Buffer, const TCHAR* SectionName)
{
	Buffer.Appendf(TEXT("<%s>" LINE_TERMINATOR_ANSI), SectionName);
}

void FGenericCrashContext::EndSection(FString& Buffer, const TCHAR* SectionName)
{
	Buffer.Appendf(TEXT("</%s>" LINE_TERMINATOR_ANSI), SectionName);
}

void FGenericCrashContext::AddSection(FString& Buffer, const TCHAR* SectionName, const FString& SectionContent)
{
	BeginSection(Buffer, SectionName);
	Buffer.Appendf(TEXT("%s"), *FXmlEscapedString(SectionContent));
	EndSection(Buffer, SectionName);
}

template<typename DEST>
static void AppendEscapedXMLString(DEST& OutBuffer, FStringView Text)
{
	if (Text.IsEmpty())
	{
		return;
	}

	for (TCHAR C : Text)
	{
		switch (C)
		{
		case TCHAR('&'):
			OutBuffer += TEXT("&amp;");
			break;
		case TCHAR('"'):
			OutBuffer += TEXT("&quot;");
			break;
		case TCHAR('\''):
			OutBuffer += TEXT("&apos;");
			break;
		case TCHAR('<'):
			OutBuffer += TEXT("&lt;");
			break;
		case TCHAR('>'):
			OutBuffer += TEXT("&gt;");
			break;
		case TCHAR('\r'):
			break;
		default:
			OutBuffer += C;
		};
	}
}

void FGenericCrashContext::AppendEscapedXMLString(FString& OutBuffer, FStringView Text)
{
	::AppendEscapedXMLString(OutBuffer, Text);
}

void FGenericCrashContext::AppendEscapedXMLString(FStringBuilderBase& OutBuffer, FStringView Text)
{
	::AppendEscapedXMLString(OutBuffer, Text);
}

FString FGenericCrashContext::UnescapeXMLString( const FString& Text )
{
	return Text
		.Replace(TEXT("&amp;"), TEXT("&"))
		.Replace(TEXT("&quot;"), TEXT("\""))
		.Replace(TEXT("&apos;"), TEXT("'"))
		.Replace(TEXT("&lt;"), TEXT("<"))
		.Replace(TEXT("&gt;"), TEXT(">"));
}

FString FGenericCrashContext::GetCrashGameName()
{
	return FString(NCached::Session.GameName);
}

const TCHAR* FGenericCrashContext::GetCrashTypeString(ECrashContextType Type)
{
	switch (Type)
	{
	case ECrashContextType::Hang:
		return CrashTypeHang;
	case ECrashContextType::GPUCrash:
		return CrashTypeGPU;
	case ECrashContextType::Ensure:
		return CrashTypeEnsure;
	case ECrashContextType::Stall:
		return CrashTypeStall;
	case ECrashContextType::Assert:
		return CrashTypeAssert;
	case ECrashContextType::AbnormalShutdown:
		return CrashTypeAbnormalShutdown;
	case ECrashContextType::OutOfMemory:
		return CrashTypeOutOfMemory;
	case ECrashContextType::VerseRuntimeError:
		return CrashTypeVerseRuntimeError;
	default:
		return CrashTypeCrash;
	}
}

const TCHAR* FGenericCrashContext::GetCrashConfigFilePath()
{
	if (FCString::Strlen(NCached::Session.CrashConfigFilePath) == 0)
	{
		FString CrashConfigFilePath = FPaths::Combine(GetCrashConfigFolder(), NCached::Session.CrashGUIDRoot, FGenericCrashContext::CrashConfigFileNameW);
		CrashConfigFilePath = IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*CrashConfigFilePath);
		NCached::Set(NCached::Session.CrashConfigFilePath, *CrashConfigFilePath);
	}
	return NCached::Session.CrashConfigFilePath;
}

const TCHAR* FGenericCrashContext::GetCrashConfigFolder()
{
	static FString CrashConfigFolder;
	if (CrashConfigFolder.IsEmpty())
	{
		CrashConfigFolder = FPaths::Combine(*FPaths::GeneratedConfigDir(), TEXT("CrashReportClient"));
	}
	return *CrashConfigFolder;
}

void FGenericCrashContext::PurgeOldCrashConfig()
{
	int32 PurgeDays = 2;
	GConfig->GetInt(ConfigSectionName, CrashConfigPurgeDays, PurgeDays, GEngineIni);

	if (PurgeDays > 0)
	{
		IFileManager& FileManager = IFileManager::Get();

		// Delete items older than PurgeDays
		TArray<FString> Directories;
		FileManager.FindFiles(Directories, *(FPaths::Combine(GetCrashConfigFolder(), CrashGUIDRootPrefix) + TEXT("*")), false, true);

		for (const FString& Dir : Directories)
		{
			const FString CrashConfigDirectory = FPaths::Combine(GetCrashConfigFolder(), *Dir);
			const FDateTime DirectoryAccessTime = FileManager.GetTimeStamp(*CrashConfigDirectory);
			if (FDateTime::Now() - DirectoryAccessTime > FTimespan::FromDays(PurgeDays))
			{
				FileManager.DeleteDirectory(*CrashConfigDirectory, false, true);
			}
		}
	}
}

void FGenericCrashContext::SetEpicAccountId(const FString& EpicAccountId)
{
	NCached::Set(NCached::Session.EpicAccountId, *EpicAccountId);
}

void FGenericCrashContext::ResetEngineData()
{
	NCached::EngineData.Reset();
	OnEngineDataReset.Broadcast();
}

void FGenericCrashContext::SetEngineData(const FString& Key, const FString& Value)
{
	if (Value.Len() == 0)
	{
		// for testing purposes, only log values when they change, but don't pay the lookup price normally.
		UE_SUPPRESS(LogCrashContext, VeryVerbose, 
		{
			if (NCached::EngineData.Find(Key))
			{
				UE_LOGF(LogCrashContext, VeryVerbose, "FGenericCrashContext::SetEngineData(%ls, <RemoveKey>)", *Key);
			}
		});
		NCached::EngineData.Remove(Key);
	}
	else
	{
		FString& OldVal = NCached::EngineData.FindOrAdd(Key);
		UE_SUPPRESS(LogCrashContext, VeryVerbose, 
		{
			if (OldVal != Value)
			{
				UE_LOGF(LogCrashContext, VeryVerbose, "FGenericCrashContext::SetEngineData(%ls, %ls)", *Key, *Value);
			}
		});
		OldVal = Value;
	}

	OnEngineDataSet.Broadcast(Key, Value);
}

void FGenericCrashContext::SetGPUBreadcrumbs(FGPUBreadcrumbCrashData&& Data)
{
	// Always prefer data from the "RHI" source
	if (!NCached::GPUBreadcrumbs.IsSet() || NCached::GPUBreadcrumbs->SourceName != TEXT("RHI") || Data.SourceName == TEXT("RHI"))
	{
		NCached::GPUBreadcrumbs.Emplace(MoveTemp(Data));
	}
}

const TMap<FString, FString>& FGenericCrashContext::GetEngineData()
{
	return NCached::EngineData;
}

/** Get arbitrary engine data from the crash context */
const FString* FGenericCrashContext::GetEngineData(const FString& Key)
{
	return NCached::EngineData.Find(Key);
}

void FGenericCrashContext::ResetGameData()
{
	NCached::GameData.Reset();
	OnGameDataReset.Broadcast();
}

void FGenericCrashContext::SetGameData(const FString& Key, const FString& Value)
{
	if (Value.Len() == 0)
	{
		// for testing purposes, only log values when they change, but don't pay the lookup price normally.
		UE_SUPPRESS(LogCrashContext, VeryVerbose, 
		{
			if (NCached::GameData.Find(Key))
			{
				UE_LOGF(LogCrashContext, VeryVerbose, "FGenericCrashContext::SetGameData(%ls, <RemoveKey>)", *Key);
			}
		});
		NCached::GameData.Remove(Key);
	}
	else
	{
		FString& OldVal = NCached::GameData.FindOrAdd(Key);
		UE_SUPPRESS(LogCrashContext, VeryVerbose, 
		{
			if (OldVal != Value)
			{
				UE_LOGF(LogCrashContext, VeryVerbose, "FGenericCrashContext::SetGameData(%ls, %ls)", *Key, *Value);
			}
		});
		OldVal = Value;
	}

	OnGameDataSet.Broadcast(Key, Value);
}

const TMap<FString, FString>& FGenericCrashContext::GetGameData()
{
	return NCached::GameData;
}

/** Get arbitrary game data from the crash context */
const FString* FGenericCrashContext::GetGameData(const FString& Key)
{
	return NCached::GameData.Find(Key);
}

void FGenericCrashContext::AddPlugin(const FString& PluginDesc)
{
	NCached::EnabledPluginsList.Add(PluginDesc);
}

void FGenericCrashContext::AddFile(const FString& AbsoluteSourcePath)
{
	FScopeLock Lock(&NCached::CriticalSection);
	NCached::FilePaths.Add(AbsoluteSourcePath);
}

FString FGenericCrashContext::DumpLog(const FString& CrashFolderAbsolute)
{
	// Copy log
	const FString LogSrcAbsolute = FPlatformOutputDevices::GetAbsoluteLogFilename();
	FString LogFilename = FPaths::GetCleanFilename(LogSrcAbsolute);
	const FString LogDstAbsolute = FPaths::Combine(*CrashFolderAbsolute, *LogFilename);

	// If we have a memory only log, make sure it's dumped to file before we attach it to the report
#if !NO_LOGGING
	bool bMemoryOnly = FPlatformOutputDevices::GetLog()->IsMemoryOnly();
	bool bBacklogEnabled = FOutputDeviceRedirector::Get()->HasBacklogForCrash();

	if (bMemoryOnly || bBacklogEnabled)
	{
		TUniquePtr<FArchive> LogFile(IFileManager::Get().CreateFileWriter(*LogDstAbsolute, FILEWRITE_AllowRead));
		if (LogFile)
		{
			if (bMemoryOnly)
			{
				FPlatformOutputDevices::GetLog()->Dump(*LogFile);
			}
			else
			{
				FCrashBacklogOutputDevice Wrapper(LogFile.Get());
				Wrapper.SetSuppressEventTag(!GCrashContextFormatEarlyCrashLog);
				TGuardValue GuardPrintLogTimes(GPrintLogTimes, ELogTimes::SinceGStartTime);
				GLog->SerializeBacklogForCrash(&Wrapper);
			}

			LogFile->Flush();
		}
	}
	else
	{
		const bool bReplace = true;
		const bool bEvenIfReadOnly = false;
		const bool bAttributes = false;
		FCopyProgress* const CopyProgress = nullptr;
		static_cast<void>(IFileManager::Get().Copy(*LogDstAbsolute, *LogSrcAbsolute, bReplace, bEvenIfReadOnly, bAttributes, CopyProgress, FILEREAD_AllowWrite, FILEWRITE_AllowRead));	// best effort, so don't care about result: couldn't copy -> tough, no log
	}
#endif // !NO_LOGGING

	return LogDstAbsolute;
}

void FGenericCrashContext::CapturePortableCallStack(void* ErrorProgramCounter, void* Context)
{
	// Capture the stack trace
	static const int StackTraceMaxDepth = 100;
	uint64 StackTrace[StackTraceMaxDepth];
	FMemory::Memzero(StackTrace);
	int32 StackTraceDepth = FPlatformStackWalk::CaptureStackBackTrace(StackTrace, StackTraceMaxDepth, Context);

	const uint64* StackTraceCursor = StackTrace;
	if (ErrorProgramCounter != nullptr)
	{
		for (int32 i = 0; i < StackTraceDepth; ++i)
		{
			if (StackTrace[i] != uint64(ErrorProgramCounter))
			{
				continue;
			}

			SetNumMinidumpFramesToIgnore(i);
			StackTraceCursor = StackTrace + i;
			StackTraceDepth -= i;
			break;
		}
	}

	// Generate the portable callstack from it
	SetPortableCallStack(StackTraceCursor, StackTraceDepth);
}

void FGenericCrashContext::CaptureThreadPortableCallStack(const uint64 ThreadId, void* Context)
{
	// Capture the stack trace
	static const int StackTraceMaxDepth = 100;
	uint64 StackTrace[StackTraceMaxDepth];
	FMemory::Memzero(StackTrace);
	int32 StackTraceDepth = FPlatformStackWalk::CaptureThreadStackBackTrace(ThreadId, StackTrace, StackTraceMaxDepth, Context);

	// Generate the portable callstack from it
	SetPortableCallStack(StackTrace, StackTraceDepth);
}

void FGenericCrashContext::SetPortableCallStack(const uint64* StackFrames, int32 NumStackFrames)
{
	GetPortableCallStack(StackFrames, NumStackFrames, CallStack);
}

void FGenericCrashContext::GetPortableCallStack(const uint64* StackFrames, int32 NumStackFrames, TArray<FCrashStackFrame>& OutCallStack) const
{
	// Get all the modules in the current process
	uint32 NumModules = (uint32)FPlatformStackWalk::GetProcessModuleCount();

	TArray<FStackWalkModuleInfo> Modules;
	Modules.AddUninitialized(NumModules);

	NumModules = FPlatformStackWalk::GetProcessModuleSignatures(Modules.GetData(), NumModules);
	Modules.SetNum(NumModules);

	// Update the callstack with offsets from each module
	OutCallStack.Reset(NumStackFrames);
	for(int32 Idx = 0; Idx < NumStackFrames; Idx++)
	{
		const uint64 StackFrame = StackFrames[Idx];

		// Try to find the module containing this stack frame
		const FStackWalkModuleInfo* FoundModule = nullptr;
		for(const FStackWalkModuleInfo& Module : Modules)
		{
			if(StackFrame >= Module.BaseOfImage && StackFrame < Module.BaseOfImage + Module.ImageSize)
			{
				FoundModule = &Module;
				break;
			}
		}

		// Add the callstack item
		if(FoundModule == nullptr)
		{
			OutCallStack.Add(FCrashStackFrame(TEXT("Unknown"), 0, StackFrame));
		}
		else
		{
			OutCallStack.Add(FCrashStackFrame(FPaths::GetBaseFilename(FoundModule->ImageName), FoundModule->BaseOfImage, StackFrame - FoundModule->BaseOfImage));
		}
	}
}

void FGenericCrashContext::AddPortableThreadCallStacks(TConstArrayView<FThreadCallStack> Threads)
{
	for (const FThreadCallStack& Thread : Threads)
	{
		AddPortableThreadCallStack(Thread.ThreadId, Thread.ThreadName, Thread.StackFrames.GetData(), Thread.StackFrames.Num());
	}
}

void FGenericCrashContext::AddPortableThreadCallStack(uint32 ThreadId, const TCHAR* ThreadName, const uint64* StackFrames, int32 NumStackFrames)
{
	// Not implemented for generic class
}

void FGenericCrashContext::CaptureModules()
{
	ModulesInfo.Reset();
	GetModules(ModulesInfo);
}

void FGenericCrashContext::GetModules(TArray<FStackWalkModuleInfo>& OutModules) const
{
	int32 Count = FPlatformStackWalk::GetProcessModuleCount();
	if (Count > 0)
	{
		OutModules.Reset();
		OutModules.AddZeroed(Count);
		Count = FPlatformStackWalk::GetProcessModuleSignatures(OutModules.GetData(), Count);
		OutModules.SetNum(Count);
	}
}

void FGenericCrashContext::GenerateFiles(const TCHAR* OutputDirectory)
{
	if (CVarTraceSnapshotOnCrash.GetValueOnAnyThread())
	{
		FString SnapshotPath = FPaths::Combine(OutputDirectory, TEXT("Snapshot.utrace"));
		FTraceAuxiliary::WriteSnapshot(*SnapshotPath, CVarTraceSnapshotOnCrashTailSize.GetValueOnAnyThread());
	}
}

void FGenericCrashContext::CopyPlatformSpecificFiles(const TCHAR* OutputDirectory, void* Context)
{
	// If present, include the crash report config file to pass config values to the CRC
	const TCHAR* CrashConfigSrcPath = GetCrashConfigFilePath();

	// Use the physical file system directly rather than IFileManager, to avoid 
	// being denied in scenarios where bAllowNonUFSIniWhenCooked=false
	IPlatformFile* PhysicalFile = FPlatformFileManager::Get().FindPlatformFile(IPlatformFile::GetPhysicalTypeName());
	if (PhysicalFile != nullptr && PhysicalFile->FileExists(CrashConfigSrcPath))
	{
		FString CrashConfigFilename = FPaths::GetCleanFilename(CrashConfigSrcPath);
		const FString CrashConfigDstAbsolute = FPaths::Combine(OutputDirectory, *CrashConfigFilename);
		PhysicalFile->CopyFile(*CrashConfigDstAbsolute, CrashConfigSrcPath);	// best effort, so don't care about result: couldn't copy -> tough, no config
	}

	for (const FString& AbsoluteSource : NCached::FilePaths)
	{
		if (IFileManager::Get().FileExists(*AbsoluteSource))
		{
			const FString AbsoluteDest = FPaths::Combine(OutputDirectory, FPaths::GetCleanFilename(AbsoluteSource));

			// Try to move file, may fail due to any number of reasons
			if (!IFileManager::Get().Move(*AbsoluteDest, *AbsoluteSource))
			{
				UE_LOGF(LogCrashContext, Error, "Error moving crash context file '%ls' to '%ls'", *AbsoluteSource, *AbsoluteDest);
			}
		}
	}
}

#if WITH_ADDITIONAL_CRASH_CONTEXTS

static FAdditionalCrashContextStack* GProviderHead = nullptr;

FAdditionalCrashContextStack& FAdditionalCrashContextStack::GetThreadContextProvider()
{
	static thread_local FAdditionalCrashContextStack ThreadContextProvider;
	return ThreadContextProvider;
}

void FAdditionalCrashContextStack::PushProvider(struct FScopedAdditionalCrashContextProvider* Provider)
{
	GetThreadContextProvider().PushProviderInternal(Provider);
}

void FAdditionalCrashContextStack::PopProvider()
{
	GetThreadContextProvider().PopProviderInternal();
}

FCriticalSection* GetAdditionalProviderLock()
{
	// Use a shared pointer to ensure that the critical section is not destroyed before the thread local object
	static TSharedPtr<FCriticalSection, ESPMode::ThreadSafe> GAdditionalProviderLock = MakeShared<FCriticalSection, ESPMode::ThreadSafe>();
	thread_local TSharedPtr<FCriticalSection, ESPMode::ThreadSafe> GAdditionalProviderLockLocal = GAdditionalProviderLock;

	return GAdditionalProviderLockLocal.Get();
}

FAdditionalCrashContextStack::FAdditionalCrashContextStack()
	: Next(nullptr)
{
	// Register by appending self to the the linked list. 
	FScopeLock _(GetAdditionalProviderLock());
	FAdditionalCrashContextStack** Current = &GProviderHead;
	while(*Current != nullptr)
	{
		Current = &((*Current)->Next);
	}
	*Current = this;
}

FAdditionalCrashContextStack::~FAdditionalCrashContextStack()
{
	// Unregister by iterating the list, replacing self with next
	// on the list.
	FScopeLock _(GetAdditionalProviderLock());
	FAdditionalCrashContextStack** Current = &GProviderHead;
	while (*Current != this)
	{
		Current = &((*Current)->Next);
	}
	*Current = this->Next;
}

void FAdditionalCrashContextStack::ExecuteProviders(FCrashContextExtendedWriter& Writer)
{
	// Attempt to lock. If a thread crashed while holding the lock
	// we could potentially deadlock here otherwise.
	FCriticalSection* AdditionalProviderLock = GetAdditionalProviderLock();
	if (AdditionalProviderLock->TryLock())
	{
		FAdditionalCrashContextStack* Provider = GProviderHead;
		while (Provider)
		{
			for (uint32 i = 0; i < Provider->StackIndex; i++)
			{
				const FScopedAdditionalCrashContextProvider* Callback = Provider->Stack[i];
				Callback->Execute(Writer);
			}
			Provider = Provider->Next;
		}
		AdditionalProviderLock->Unlock();
	}
}

struct FCrashContextExtendedWriterImpl : public FCrashContextExtendedWriter
{
	FCrashContextExtendedWriterImpl(const TCHAR* InOutputDirectory)
		: OutputDirectory(InOutputDirectory)
	{
	}
	
	void OutputBuffer(const TCHAR* Identifier, const uint8* Data, uint32 DataSize, const TCHAR* Extension)
	{
		TCHAR Filename[1024] = { 0 };
		FCString::Snprintf(Filename, 1024, TEXT("%s/%s.%s"), OutputDirectory, Identifier, Extension);
		TUniquePtr<IFileHandle> File(IPlatformFile::GetPlatformPhysical().OpenWrite(Filename));
		if (File)
		{
			File->Write(Data, DataSize);
			File->Flush();
		}
	}

	virtual void AddBuffer(const TCHAR* Identifier, const uint8* Data, uint32 DataSize) override
	{
		if (Identifier == nullptr || Data == nullptr || DataSize == 0)
		{
			return;
		}

		OutputBuffer(Identifier, Data, DataSize, TEXT("bin"));
	}

	virtual void AddString(const TCHAR* Identifier, const TCHAR* DataStr) override
	{
		if (Identifier == nullptr || DataStr == nullptr)
		{
			return;
		}

		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Additional Crash Context (Key=\"%s\", Value=\"%s\")"), Identifier, DataStr);

		FTCHARToUTF8 Converter(DataStr);
		OutputBuffer(Identifier, (uint8*)Converter.Get(), Converter.Length(), TEXT("txt"));
	}

	/** MoveFile a file into the report */
	virtual bool MoveFile(const TCHAR* FromFilePath) override
	{
		TCHAR DestFilePath[1024] = { 0 };
		bool bFileMoved = false;
		FString ToFileName = *FPaths::GetCleanFilename(FromFilePath);
		int32 DestPathLength = FCString::Snprintf(DestFilePath, UE_ARRAY_COUNT(DestFilePath), TEXT("%s/%s"), OutputDirectory, *ToFileName);
		if (DestPathLength>0 && DestPathLength < UE_ARRAY_COUNT(DestFilePath))
		{
			IPlatformFile& PlatformFile = IPlatformFile::GetPlatformPhysical();
			bFileMoved = PlatformFile.MoveFile(DestFilePath, FromFilePath);
			if (!bFileMoved) // attempt copy the file if the move fails. (cant move across file systems etc)
			{
				bFileMoved = PlatformFile.CopyFile(DestFilePath, FromFilePath);
				if (bFileMoved)
				{
					PlatformFile.DeleteFile(FromFilePath);
				}
				else
				{
					FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Crash Context FileMove failed. (%s , %s)"), FromFilePath, DestFilePath);
				}
			}
		}
		return bFileMoved;
	}

private:
	const TCHAR* OutputDirectory;
};

#endif //WITH_ADDITIONAL_CRASH_CONTEXTS 

void FGenericCrashContext::DumpAdditionalContext(const TCHAR* CrashFolderAbsolute)
{
	FAdditionalCrashContextParams Params;
	DumpAdditionalContext(CrashFolderAbsolute, Params);
}

void FGenericCrashContext::DumpAdditionalContext(const TCHAR* CrashFolderAbsolute, const FAdditionalCrashContextParams& Params)
{
#if WITH_ADDITIONAL_CRASH_CONTEXTS 
	FCrashContextExtendedWriterImpl Writer(CrashFolderAbsolute);
	if (!UE::IsGCLocked())
	{
		AdditionalCrashContextDelegate.Broadcast(Writer);
		AdditionalCrashContextDelegateEx.Broadcast(Writer, Params);
	}
	FAdditionalCrashContextStack::ExecuteProviders(Writer);
#endif
}

/**
 * Attempts to create the output report directory.
 */
bool FGenericCrashContext::CreateCrashReportDirectory(const TCHAR* CrashGUIDRoot, int32 CrashIndex, FString& OutCrashDirectoryAbsolute)
{
	// Generate Crash GUID
	TCHAR CrashGUID[FGenericCrashContext::CrashGUIDLength];
	FCString::Snprintf(CrashGUID, FGenericCrashContext::CrashGUIDLength, TEXT("%s_%04i"), CrashGUIDRoot, CrashIndex);

	// The FPaths commands usually checks for command line override, if FCommandLine not yet
	// initialized we cannot create a directory. Also there is no way of knowing if the file manager
	// has been created.
	if (!FCommandLine::IsInitialized())
	{
		return false;
	}

	FString CrashFolder = FPaths::Combine(*FPaths::ProjectSavedDir(), TEXT("Crashes"), CrashGUID);
	OutCrashDirectoryAbsolute = IFileManager::Get().ConvertToAbsolutePathForExternalAppForWrite(*CrashFolder);
	return IFileManager::Get().MakeDirectory(*OutCrashDirectoryAbsolute, true);
}

FString RecoveryService::GetRecoveryServerName()
{
	// Requirements: To avoid collision, the name must be unique on the local machine (multiple instances) and across the local network (multiple users).
	static FGuid RecoverySessionGuid = FGuid::NewGuid();
	return RecoverySessionGuid.ToString();
}

FString RecoveryService::MakeSessionName()
{
	// Convention: The session name starts with the server name (uniqueness), followed by a zero-based unique sequence number (idendify the latest session reliably), the the session creation time and the project name.
	static TAtomic<int32> SessionNum(0);
	return FString::Printf(TEXT("%s_%d_%s_%s"), *RecoveryService::GetRecoveryServerName(), SessionNum++, *FDateTime::UtcNow().ToString(), FApp::GetProjectName());
}

bool RecoveryService::TokenizeSessionName(const FString& SessionName, FString* OutServerName, int32* SeqNum, FString* ProjName, FDateTime* DateTime)
{
	// Parse a sessionName created with 'MakeSessionName()' that have the following format: C6EACAD6419AF672D75E2EA91E05BF55_1_2019.12.05-08.59.03_FP_FirstPerson
	//     ServerName = C6EACAD6419AF672D75E2EA91E05BF55
	//     SeqNum = 1
	//     DateTime = 2019.12.05-08.59.03
	//     ProjName = FP_FirstPerson
	FRegexPattern Pattern(TEXT(R"((^[A-Z0-9]+)_([0-9])+_([0-9\.-]+)_(.+))")); // Need help with regex? Try https://regex101.com/
	FRegexMatcher Matcher(Pattern, SessionName);

	if (!Matcher.FindNext())
	{
		return false; // Failed to parse.
	}
	if (OutServerName)
	{
		*OutServerName = Matcher.GetCaptureGroup(1);
	}
	if (SeqNum)
	{
		LexFromString(*SeqNum, *Matcher.GetCaptureGroup(2));
	}
	if (ProjName)
	{
		*ProjName = Matcher.GetCaptureGroup(4);
	}
	if (DateTime)
	{
		return FDateTime::Parse(Matcher.GetCaptureGroup(3), *DateTime);
	}

	return true; // Successfully parsed.
}

