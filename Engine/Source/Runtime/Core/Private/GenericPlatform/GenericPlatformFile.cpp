// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenericPlatform/GenericPlatformFile.h"
#include "Algo/Accumulate.h"
#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "Misc/PathViews.h"
#include "HAL/PlatformMisc.h"
#include "HAL/ThreadSafeCounter.h"
#include "Stats/Stats.h"
#include "Async/AsyncWork.h"
#include "Templates/UniquePtr.h"
#include "Misc/ScopeLock.h"
#include "HAL/LowLevelMemTracker.h"
#include "String/BytesToHex.h"

#include "Async/AsyncFileHandle.h"
#include "Async/MappedFileHandle.h"
#include "Async/ParallelFor.h"
#include "Misc/ScopeRWLock.h"

class FGenericBaseRequest;
class FGenericAsyncReadFileHandle;

class FGenericReadRequestWorker : public FNonAbandonableTask
{
	FGenericBaseRequest& ReadRequest;
public:
	FGenericReadRequestWorker(FGenericBaseRequest* InReadRequest)
		: ReadRequest(*InReadRequest)
	{
	}
	void DoWork();
	FORCEINLINE TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FGenericReadRequestWorker, STATGROUP_ThreadPoolAsyncTasks);
	}
};

class FGenericBaseRequest : public IAsyncReadRequest
{
protected:
	FAsyncTask<FGenericReadRequestWorker>* Task;
	IPlatformFile* LowerLevel;
	const TCHAR* Filename;
public:
	FGenericBaseRequest(IPlatformFile* InLowerLevel, const TCHAR* InFilename, FAsyncFileCallBack* CompleteCallback, bool bInSizeRequest, uint8* UserSuppliedMemory = nullptr)
		: IAsyncReadRequest(CompleteCallback, bInSizeRequest, UserSuppliedMemory)
		, Task(nullptr)
		, LowerLevel(InLowerLevel)
		, Filename(InFilename)
	{
	}

	void Start()
	{
		if (FPlatformProcess::SupportsMultithreading())
		{
			Task->StartBackgroundTask(GIOThreadPool);
		}
		else
		{
			Task->StartSynchronousTask();
			WaitCompletionImpl(0.0f); // might as well finish it now
		}
	}

	virtual ~FGenericBaseRequest()
	{
		if (Task)
		{
			Task->EnsureCompletion(); // if the user polls, then we might never actual sync completion of the task until now, this will almost always be done, however we need to be sure the task is clear
			delete Task; 
		}
	}

	virtual void PerformRequest() = 0;

	virtual void WaitCompletionImpl(float TimeLimitSeconds) override TSAN_SAFE
	{
		if (Task)
		{
			bool bResult;
			if (TimeLimitSeconds <= 0.0f)
			{
				Task->EnsureCompletion();
				bResult = true;
			}
			else
			{
				bResult = Task->WaitCompletionWithTimeout(TimeLimitSeconds);
			}
			if (bResult)
			{
				check(bCompleteAndCallbackCalled);
				delete Task;
				Task = nullptr;
			}
		}
	}

	virtual void EnsureCompletion() override
	{
		if (Task)
		{
			Task->EnsureCompletion();
			delete Task;
			Task = nullptr;
		}
	}

	virtual void CancelImpl() override
	{
		if (Task)
		{
			if (Task->Cancel())
			{
				delete Task;
				Task = nullptr;
				SetComplete();
			}
		}
	}
};

class FGenericSizeRequest : public FGenericBaseRequest
{
public:
	FGenericSizeRequest(IPlatformFile* InLowerLevel, const TCHAR* InFilename, FAsyncFileCallBack* CompleteCallback)
		: FGenericBaseRequest(InLowerLevel, InFilename, CompleteCallback, true)
	{
		Task = new FAsyncTask<FGenericReadRequestWorker>(this);
		Start();
	}
	virtual void PerformRequest() override
	{
		if (!bCanceled)
		{
			check(LowerLevel && Filename);
			Size = LowerLevel->FileSize(Filename);
		}
		SetComplete();
	}

protected:
	virtual void ReleaseMemoryOwnershipImpl() override
	{
	}
};

class FGenericReadRequest : public FGenericBaseRequest
{
	FGenericAsyncReadFileHandle* Owner;
	int64 Offset;
	int64 BytesToRead;
	EAsyncIOPriorityAndFlags PriorityAndFlags;
public:
	FGenericReadRequest(FGenericAsyncReadFileHandle* InOwner, IPlatformFile* InLowerLevel, const TCHAR* InFilename, FAsyncFileCallBack* CompleteCallback, uint8* UserSuppliedMemory, int64 InOffset, int64 InBytesToRead, EAsyncIOPriorityAndFlags InPriorityAndFlags)
		: FGenericBaseRequest(InLowerLevel, InFilename, CompleteCallback, false, UserSuppliedMemory)
		, Owner(InOwner)
		, Offset(InOffset)
		, BytesToRead(InBytesToRead)
		, PriorityAndFlags(InPriorityAndFlags)
	{
		check(Offset >= 0 && BytesToRead > 0);
		if (CheckForPrecache()) 
		{
			SetComplete();
		}
		else
		{
			Task = new FAsyncTask<FGenericReadRequestWorker>(this);
			Start();
		}
	}
	~FGenericReadRequest();
	bool CheckForPrecache();
	virtual void PerformRequest() override;
	uint8* GetContainedSubblock(uint8* UserSuppliedMemory, int64 InOffset, int64 InBytesToRead)
	{
		if (InOffset >= Offset && InOffset + InBytesToRead <= Offset + BytesToRead &&
			this->PollCompletion() && Memory)
		{
			check(Memory);
			if (!UserSuppliedMemory)
			{
				UserSuppliedMemory = (uint8*)FMemory::Malloc(InBytesToRead);
				INC_MEMORY_STAT_BY(STAT_AsyncFileMemory, InBytesToRead);
			}
			FMemory::Memcpy(UserSuppliedMemory, Memory + InOffset - Offset, InBytesToRead);
			return UserSuppliedMemory;
		}
		return nullptr;
	}

protected:
	virtual void ReleaseMemoryOwnershipImpl() override
	{
		DEC_MEMORY_STAT_BY(STAT_AsyncFileMemory, BytesToRead);
	}
};

void FGenericReadRequestWorker::DoWork()
{
	ReadRequest.PerformRequest();
}

// @todo switch et al: this is a temporary measure until we can track down some threaded file handling issues on Switch 
#if (PLATFORM_IOS || PLATFORM_MAC)
#define DISABLE_HANDLE_CACHING (1)
#else
#define DISABLE_HANDLE_CACHING (0)
#endif

#if WITH_EDITOR
#define MAX_CACHED_SYNC_FILE_HANDLES_PER_GENERIC_ASYNC_FILE_HANDLE 1
#define FORCE_SINGLE_SYNC_FILE_HANDLE_PER_GENERIC_ASYNC_FILE_HANDLE 1
#define DISABLE_BUFFERING_ON_GENERIC_ASYNC_FILE_HANDLE 0
#else
#define MAX_CACHED_SYNC_FILE_HANDLES_PER_GENERIC_ASYNC_FILE_HANDLE PLATFORM_MAX_CACHED_SYNC_FILE_HANDLES_PER_GENERIC_ASYNC_FILE_HANDLE
#define FORCE_SINGLE_SYNC_FILE_HANDLE_PER_GENERIC_ASYNC_FILE_HANDLE PLATFORM_FORCE_SINGLE_SYNC_FILE_HANDLE_PER_GENERIC_ASYNC_FILE_HANDLE
#define DISABLE_BUFFERING_ON_GENERIC_ASYNC_FILE_HANDLE 1
#endif

static int32 GCacheHandleForPakFilesOnly = 1;
static FAutoConsoleVariableRef CVarCacheHandleForPakFilesOnly(
	TEXT("AsyncReadFile.CacheHandleForPakFilesOnly"),
	GCacheHandleForPakFilesOnly,
	TEXT("Control how Async read handle caches the underlying platform handle for files.\n"
	     "0: Cache the underlying platform handles for all files.\n"
	     "1: Cache the underlying platform handle for .pak files only (default).\n"),
	ECVF_Default
);

class FGenericAsyncReadFileHandle final : public IAsyncReadFileHandle
{
	IPlatformFile* LowerLevel;
	FString Filename;
	TArray<FGenericReadRequest*> LiveRequests; // linear searches could be improved

	FCriticalSection LiveRequestsCritical;
	FCriticalSection HandleCacheCritical;
	IFileHandle* HandleCache[MAX_CACHED_SYNC_FILE_HANDLES_PER_GENERIC_ASYNC_FILE_HANDLE];
	bool bOpenFailed;
	bool bDisableHandleCaching;
	bool bAllowWrite;
public:
	FGenericAsyncReadFileHandle(IPlatformFile* InLowerLevel, const TCHAR* InFilename, bool bInAllowWrite)
		: LowerLevel(InLowerLevel)
		, Filename(InFilename)
		, bOpenFailed(false)
		, bDisableHandleCaching(!!DISABLE_HANDLE_CACHING)
		, bAllowWrite(bInAllowWrite)
	{
#if !WITH_EDITOR
		if (GCacheHandleForPakFilesOnly && !Filename.EndsWith(TEXT(".pak")))
		{
			bDisableHandleCaching = true; // Closing files can be slow, so we want to do that on the thread and not on the calling thread. Pak files are rarely, if ever, closed and that is where the handle caching saves.
		}
#endif
		for (int32 Index = 0; Index < MAX_CACHED_SYNC_FILE_HANDLES_PER_GENERIC_ASYNC_FILE_HANDLE; Index++)
		{
			HandleCache[Index] = nullptr;
		}
	}
	~FGenericAsyncReadFileHandle()
	{
		FScopeLock Lock(&LiveRequestsCritical);
		check(!LiveRequests.Num()); // must delete all requests before you delete the handle
		for (int32 Index = 0; Index < MAX_CACHED_SYNC_FILE_HANDLES_PER_GENERIC_ASYNC_FILE_HANDLE; Index++)
		{
			if (HandleCache[Index])
			{
				delete HandleCache[Index];
			}
		}
	}
	void RemoveRequest(FGenericReadRequest* Req)
	{
		FScopeLock Lock(&LiveRequestsCritical);
		verify(LiveRequests.Remove(Req) == 1);
	}
	uint8* GetPrecachedBlock(uint8* UserSuppliedMemory, int64 InOffset, int64 InBytesToRead)
	{
		FScopeLock Lock(&LiveRequestsCritical);
		uint8* Result = nullptr;
		for (FGenericReadRequest* Req : LiveRequests)
		{
			Result = Req->GetContainedSubblock(UserSuppliedMemory, InOffset, InBytesToRead);
			if (Result)
			{
				break;
			}
		}
		return Result;
	}
	virtual IAsyncReadRequest* SizeRequest(FAsyncFileCallBack* CompleteCallback = nullptr) override
	{
		return new FGenericSizeRequest(LowerLevel, *Filename, CompleteCallback);
	}
	virtual IAsyncReadRequest* ReadRequest(int64 Offset, int64 BytesToRead, EAsyncIOPriorityAndFlags PriorityAndFlags = AIOP_Normal, FAsyncFileCallBack* CompleteCallback = nullptr, uint8* UserSuppliedMemory = nullptr) override
	{
		FGenericReadRequest* Result = new FGenericReadRequest(this, LowerLevel, *Filename, CompleteCallback, UserSuppliedMemory, Offset, BytesToRead, PriorityAndFlags);
		if (PriorityAndFlags & AIOP_FLAG_PRECACHE) // only precache requests are tracked for possible reuse
		{
			FScopeLock Lock(&LiveRequestsCritical);
			LiveRequests.Add(Result);
		}
		return Result;
	}

	virtual void ShrinkHandleBuffers() override
	{
		if (!bDisableHandleCaching)
		{
			FScopeLock Lock(&HandleCacheCritical);
			for (int32 Index = 0; Index < MAX_CACHED_SYNC_FILE_HANDLES_PER_GENERIC_ASYNC_FILE_HANDLE; Index++)
			{
				if (HandleCache[Index])
				{
					HandleCache[Index]->ShrinkBuffers();
				}
			}
		}
	}

	IFileHandle* GetHandle()
	{
		if (bDisableHandleCaching)
		{
#if DISABLE_BUFFERING_ON_GENERIC_ASYNC_FILE_HANDLE
			return LowerLevel->OpenReadNoBuffering(*Filename, bAllowWrite);
#else
			return LowerLevel->OpenRead(*Filename, bAllowWrite);
#endif
		}
		IFileHandle* Result = nullptr;
		if (FORCE_SINGLE_SYNC_FILE_HANDLE_PER_GENERIC_ASYNC_FILE_HANDLE)
		{
			check(MAX_CACHED_SYNC_FILE_HANDLES_PER_GENERIC_ASYNC_FILE_HANDLE == 1);
			HandleCacheCritical.Lock();
			if (!HandleCache[0] && !bOpenFailed)
			{
#if DISABLE_BUFFERING_ON_GENERIC_ASYNC_FILE_HANDLE
				HandleCache[0] = LowerLevel->OpenReadNoBuffering(*Filename, bAllowWrite);
#else
				HandleCache[0] = LowerLevel->OpenRead(*Filename, bAllowWrite);
#endif
				bOpenFailed = (HandleCache[0] == nullptr);
			}
			Result = HandleCache[0];
			if (!Result)
			{
				HandleCacheCritical.Unlock(); // they won't free a null handle so we unlock now
			}
		}
		else
		{
			FScopeLock Lock(&HandleCacheCritical);
			for (int32 Index = 0; Index < MAX_CACHED_SYNC_FILE_HANDLES_PER_GENERIC_ASYNC_FILE_HANDLE; Index++)
			{
				if (HandleCache[Index])
				{
					Result = HandleCache[Index];
					HandleCache[Index] = nullptr;
					break;
				}
			}
			if (!Result && !bOpenFailed)
			{
#if DISABLE_BUFFERING_ON_GENERIC_ASYNC_FILE_HANDLE
				Result = LowerLevel->OpenReadNoBuffering(*Filename, bAllowWrite);
#else
				Result = LowerLevel->OpenRead(*Filename, bAllowWrite);
#endif
				bOpenFailed = (Result == nullptr);
			}
		}
		return Result;
	}

	void FreeHandle(IFileHandle* Handle)
	{
		if (!bDisableHandleCaching)
		{
			check(!bOpenFailed);
			if (FORCE_SINGLE_SYNC_FILE_HANDLE_PER_GENERIC_ASYNC_FILE_HANDLE)
			{
				check(Handle && Handle == HandleCache[0]);
				HandleCacheCritical.Unlock();
				FPlatformProcess::Sleep(0.0f); // we hope this allows some other thread waiting for this lock to wake up (at our expense) to keep the disk at near 100% utilization
				return;
			}
			{
				FScopeLock Lock(&HandleCacheCritical);
				for (int32 Index = 0; Index < MAX_CACHED_SYNC_FILE_HANDLES_PER_GENERIC_ASYNC_FILE_HANDLE; Index++)
				{
					if (!HandleCache[Index])
					{
						HandleCache[Index] = Handle;
						return;
					}
				}
			}
		}
		delete Handle;
	}

};

FGenericReadRequest::~FGenericReadRequest()
{
	if (Memory)
	{
		// this can happen with a race on cancel, it is ok, they didn't take the memory, free it now
		if (!bUserSuppliedMemory)
		{
			DEC_MEMORY_STAT_BY(STAT_AsyncFileMemory, BytesToRead);
			FMemory::Free(Memory);
		}
		Memory = nullptr;
	}
	if (PriorityAndFlags & AIOP_FLAG_PRECACHE) // only precache requests are tracked for possible reuse
	{
		Owner->RemoveRequest(this);
	}
	Owner = nullptr;
}

void FGenericReadRequest::PerformRequest()
{
	LLM_SCOPE(ELLMTag::FileSystem);

	if (!bCanceled)
	{
		bool bMemoryHasBeenAcquired = bUserSuppliedMemory;
		if (FORCE_SINGLE_SYNC_FILE_HANDLE_PER_GENERIC_ASYNC_FILE_HANDLE && !bMemoryHasBeenAcquired && BytesToRead != MAX_uint64)
		{
			// If possible, we do the malloc before we get the handle which will lock. Memory allocation can take time and other locks, so best do this before we get the file handle
			check(!Memory);
			Memory = (uint8*)FMemory::Malloc(BytesToRead);
			INC_MEMORY_STAT_BY(STAT_AsyncFileMemory, BytesToRead);
			bMemoryHasBeenAcquired = true;
		}
		IFileHandle* Handle = Owner->GetHandle();
		bCanceled = (Handle == nullptr);
		if (Handle)
		{
			if (BytesToRead == MAX_int64)
			{
				BytesToRead = Handle->Size() - Offset;
				check(BytesToRead > 0);
			}
			if (!bMemoryHasBeenAcquired)
			{
				check(!Memory);
				Memory = (uint8*)FMemory::Malloc(BytesToRead);
				INC_MEMORY_STAT_BY(STAT_AsyncFileMemory, BytesToRead);
			}
			check(Memory);
			Handle->Seek(Offset);
			Handle->Read(Memory, BytesToRead);
			Owner->FreeHandle(Handle);
		}
		else if (!bUserSuppliedMemory && bMemoryHasBeenAcquired)
		{
			check(FORCE_SINGLE_SYNC_FILE_HANDLE_PER_GENERIC_ASYNC_FILE_HANDLE);
			// oops, we allocated memory and we couldn't open the file anyway
			check(Memory);
			FMemory::Free(Memory);
			DEC_MEMORY_STAT_BY(STAT_AsyncFileMemory, BytesToRead);
			Memory = nullptr;
		}
	}
	SetComplete();
}

bool FGenericReadRequest::CheckForPrecache()
{
	if ( ( PriorityAndFlags & AIOP_FLAG_PRECACHE ) == 0 )  // only non-precache requests check for existing blocks to copy from 
	{
		check(!Memory || bUserSuppliedMemory);
		uint8* Result = Owner->GetPrecachedBlock(Memory, Offset, BytesToRead);
		if (Result)
		{
			check(!bUserSuppliedMemory || Memory == Result);
			Memory = Result;
			return true;
		}
	}
	return false;
}

IPlatformFile::IPlatformFile() = default;
IPlatformFile::~IPlatformFile() = default;

FFileOpenResult IPlatformFile::OpenRead(const TCHAR* Filename, IPlatformFile::EOpenReadFlags Flags)
{
	if (EnumHasAnyFlags(Flags, EOpenReadFlags::AllowDelete))
	{
		return MakeError(TEXT("The current IPlatformFile implementation does not support the EReadFlags::AllowDelete flag"));
	}

	const bool bWrite = EnumHasAnyFlags(Flags, EOpenReadFlags::AllowWrite);

	IFileHandle* Handle = OpenRead(Filename, bWrite);
	if (Handle != nullptr)
	{
		return MakeValue(Handle);
	}
	else
	{
		return MakeError(TEXT("[Unknown Error] Current IPlatformFile implementation does not support returning error messages"));
	}
}

FFileOpenAsyncResult IPlatformFile::OpenAsyncRead(const TCHAR* Filename, EOpenReadFlags Flags)
{
	if (EnumHasAnyFlags(Flags, EOpenReadFlags::AllowDelete))
	{
		return MakeError(TEXT("The current IPlatformFile implementation does not support the EReadFlags::AllowDelete flag"));
	}

	const bool bWrite = EnumHasAnyFlags(Flags, EOpenReadFlags::AllowWrite);

	IAsyncReadFileHandle* Handle = OpenAsyncRead(Filename, bWrite);
	if (Handle != nullptr)
	{
		return MakeValue(Handle);
	}

	return MakeError(TEXT("[Unknown Error] Current IPlatformFile implementation does not support returning error messages"));
}

IAsyncReadFileHandle* IPlatformFile::OpenAsyncRead(const TCHAR* Filename, bool bAllowWrite /*= false*/)
{
	return new FGenericAsyncReadFileHandle(this, Filename, bAllowWrite);
}

FOpenMappedResult IPlatformFile::OpenMappedEx(const TCHAR* Filename, EOpenReadFlags OpenOptions, int64 MaximumSize)
{
	// Note: cannot forward to OpenMappedEx2 here. OpenMappedEx has inconsistent behavior on different platforms,
	// so the call must be forwarded in the platform implementation

	return MakeError(TEXT("OpenMappedEx is not implemented on this platform"));
}

FOpenMappedResult IPlatformFile::OpenMappedEx2(const TCHAR* Filename, EOpenMappedFlags OpenOptions, int64 MaximumSize)
{
	return MakeError(TEXT("OpenMappedEx2 is not implemented on this platform"));
}

IFileHandle* IPlatformFile::OpenWrite(const TCHAR* Filename, bool bAppend, bool bAllowRead)
{
	EOpenWriteFlags Flags = EOpenWriteFlags::None;
	if (bAppend)
	{
		Flags |= EOpenWriteFlags::Append;
	}
	if (bAllowRead)
	{
		Flags |= EOpenWriteFlags::AllowRead;
	}

	FFileOpenResult Result = OpenWrite(Filename, Flags);
	if (Result.HasValue())
	{
		return Result.GetValue().Release();
	}

	return nullptr;
}

FFileOpenResult IPlatformFile::OpenWrite(const TCHAR* Filename, EOpenWriteFlags Flags)
{
	const bool bAppend = EnumHasAnyFlags(Flags, EOpenWriteFlags::Append);
	const bool bAllowRead = EnumHasAnyFlags(Flags, EOpenWriteFlags::AllowRead);

	if (IFileHandle* Handle = OpenWrite(Filename, bAppend, bAllowRead); Handle != nullptr)
	{
		return MakeValue(Handle);
	}

	const uint32 LastError = FPlatformMisc::GetLastError();
	return MakeError(FString::Printf(TEXT("Failed to open file '%s' for writing"), Filename != nullptr ? Filename : TEXT("<Null>")), int32(LastError));
}

DEFINE_STAT(STAT_AsyncFileMemory);
DEFINE_STAT(STAT_AsyncFileHandles);
DEFINE_STAT(STAT_AsyncFileRequests);

DEFINE_STAT(STAT_MappedFileMemory);
DEFINE_STAT(STAT_MappedFileHandles);
DEFINE_STAT(STAT_MappedFileRegions);

int64 IFileHandle::Size()
{
	int64 Current = Tell();
	SeekFromEnd();
	int64 Result = Tell();
	Seek(Current);
	return Result;
}

const TCHAR* IPlatformFile::GetPhysicalTypeName()
{
	return TEXT("PhysicalFile");
}

void IPlatformFile::GetTimeStampPair(const TCHAR* PathA, const TCHAR* PathB, FDateTime& OutTimeStampA, FDateTime& OutTimeStampB)
{
	if (GetLowerLevel())
	{
		GetLowerLevel()->GetTimeStampPair(PathA, PathB, OutTimeStampA, OutTimeStampB);
	}
	else
	{
		OutTimeStampA = GetTimeStamp(PathA);
		OutTimeStampB = GetTimeStamp(PathB);
	}
}

FDateTime IPlatformFile::GetTimeStampLocal(const TCHAR* Filename)
{
	FDateTime FileTimeStamp = GetTimeStamp(Filename);

	//Turn UTC into local
	FTimespan UTCOffset = FDateTime::Now() - FDateTime::UtcNow();
	FileTimeStamp += UTCOffset;

	return FileTimeStamp;
}

bool IPlatformFile::FDirectoryVisitor::CallShouldVisitAndVisit(const TCHAR* FilenameOrDirectory, bool bIsDirectory)
{
	FStringView LeafPathname = FPathViews::GetCleanFilename(FilenameOrDirectory);
	if (!ShouldVisitLeafPathname(LeafPathname))
	{
		return true; // Continue iterating
	}
	return Visit(FilenameOrDirectory, bIsDirectory);
}

bool IPlatformFile::FDirectoryStatVisitor::CallShouldVisitAndVisit(const TCHAR* FilenameOrDirectory,
	const FFileStatData& StatData)
{
	FStringView LeafPathname = FPathViews::GetCleanFilename(FilenameOrDirectory);
	if (!ShouldVisitLeafPathname(LeafPathname))
	{
		return true; // Continue iterating
	}
	return Visit(FilenameOrDirectory, StatData);
}

class FDirectoryVisitorFuncWrapper : public IPlatformFile::FDirectoryVisitor
{
public:
	IPlatformFile::FDirectoryVisitorFunc VisitorFunc;
	FDirectoryVisitorFuncWrapper(IPlatformFile::FDirectoryVisitorFunc InVisitorFunc)
		: VisitorFunc(InVisitorFunc)
	{
	}
	virtual bool Visit(const TCHAR* FilenameOrDirectory, bool bIsDirectory) override
	{
		return VisitorFunc(FilenameOrDirectory, bIsDirectory);
	}
};

class FDirectoryStatVisitorFuncWrapper : public IPlatformFile::FDirectoryStatVisitor
{
public:
	IPlatformFile::FDirectoryStatVisitorFunc VisitorFunc;
	FDirectoryStatVisitorFuncWrapper(IPlatformFile::FDirectoryStatVisitorFunc InVisitorFunc)
		: VisitorFunc(InVisitorFunc)
	{
	}
	virtual bool Visit(const TCHAR* FilenameOrDirectory, const FFileStatData& StatData) override
	{
		return VisitorFunc(FilenameOrDirectory, StatData);
	}
};

bool IPlatformFile::IterateDirectory(const TCHAR* Directory, FDirectoryVisitorFunc Visitor)
{
	FDirectoryVisitorFuncWrapper VisitorFuncWrapper(Visitor);
	return IterateDirectory(Directory, VisitorFuncWrapper);
}

bool IPlatformFile::IterateDirectoryStat(const TCHAR* Directory, FDirectoryStatVisitorFunc Visitor)
{
	FDirectoryStatVisitorFuncWrapper VisitorFuncWrapper(Visitor);
	return IterateDirectoryStat(Directory, VisitorFuncWrapper);
}

bool IPlatformFile::IterateDirectoryRecursively(const TCHAR* Directory, FDirectoryVisitor& Visitor)
{
	class FRecurse : public FDirectoryVisitor
	{
	public:
		FDirectoryVisitor&  Visitor;
		TArray<FString>&    Directories;
		FRecurse(FDirectoryVisitor& InVisitor, TArray<FString>& InDirectories)
			: FDirectoryVisitor(InVisitor.DirectoryVisitorFlags)
			, Visitor(InVisitor)
			, Directories(InDirectories)
		{
		}
		virtual bool Visit(const TCHAR* FilenameOrDirectory, bool bIsDirectory) override
		{
			bool bResult = Visitor.CallShouldVisitAndVisit(FilenameOrDirectory, bIsDirectory);
			if (bResult && bIsDirectory)
			{
				Directories.Emplace(FilenameOrDirectory);
			}
			return bResult;
		}
	};

	TArray<FString> DirectoriesToVisit;
	DirectoriesToVisit.Add(Directory);

	constexpr int32 MinBatchSize = 1;
	const EParallelForFlags ParallelForFlags = FTaskGraphInterface::IsRunning() && Visitor.IsThreadSafe()
		? EParallelForFlags::Unbalanced : EParallelForFlags::ForceSingleThread;
	std::atomic<bool> bResult{true};
	TArray<TArray<FString>> DirectoriesToVisitNext;
	while (bResult && DirectoriesToVisit.Num() > 0)
	{
		ParallelForWithTaskContext(TEXT("IterateDirectoryRecursively.PF"),
			DirectoriesToVisitNext,
			DirectoriesToVisit.Num(),
			MinBatchSize,
			[this, &Visitor, &DirectoriesToVisit, &bResult](TArray<FString>& Directories, int32 Index)
			{
				FRecurse Recurse(Visitor, Directories);
				if (bResult.load(std::memory_order_relaxed) && !IterateDirectory(*DirectoriesToVisit[Index], Recurse))
				{
					bResult.store(false, std::memory_order_relaxed);
				}
			},
			ParallelForFlags);
		DirectoriesToVisit.Reset(Algo::TransformAccumulate(DirectoriesToVisitNext, &TArray<FString>::Num, 0));
		for (TArray<FString>& Directories : DirectoriesToVisitNext)
		{
			DirectoriesToVisit.Append(MoveTemp(Directories));
		}
	}

	return bResult;
}

bool IPlatformFile::IterateDirectoryStatRecursively(const TCHAR* Directory, FDirectoryStatVisitor& Visitor)
{
	class FStatRecurse : public FDirectoryStatVisitor
	{
	public:
		IPlatformFile&			PlatformFile;
		FDirectoryStatVisitor&	Visitor;
		FStatRecurse(IPlatformFile&	InPlatformFile, FDirectoryStatVisitor& InVisitor)
			: PlatformFile(InPlatformFile)
			, Visitor(InVisitor)
		{
		}
		virtual bool Visit(const TCHAR* FilenameOrDirectory, const FFileStatData& StatData) override
		{
			bool bResult = Visitor.CallShouldVisitAndVisit(FilenameOrDirectory, StatData);
			if (bResult && StatData.bIsDirectory)
			{
				bResult = PlatformFile.IterateDirectoryStat(FilenameOrDirectory, *this);
			}
			return bResult;
		}
	};
	FStatRecurse Recurse(*this, Visitor);
	return IterateDirectoryStat(Directory, Recurse);
}

bool IPlatformFile::IterateDirectoryRecursively(const TCHAR* Directory, FDirectoryVisitorFunc Visitor)
{
	FDirectoryVisitorFuncWrapper VisitorFuncWrapper(Visitor);
	return IterateDirectoryRecursively(Directory, VisitorFuncWrapper);
}

bool IPlatformFile::IterateDirectoryStatRecursively(const TCHAR* Directory, FDirectoryStatVisitorFunc Visitor)
{
	FDirectoryStatVisitorFuncWrapper VisitorFuncWrapper(Visitor);
	return IterateDirectoryStatRecursively(Directory, VisitorFuncWrapper);
}

class FFindFilesVisitor : public IPlatformFile::FDirectoryVisitor
{
public:
	IPlatformFile&   PlatformFile;
	FRWLock          FoundFilesLock;
	TArray<FString>& FoundFiles;
	const TCHAR*     FileExtension;
	int32            FileExtensionLen;
	FFindFilesVisitor(IPlatformFile& InPlatformFile, TArray<FString>& InFoundFiles, const TCHAR* InFileExtension)
		: IPlatformFile::FDirectoryVisitor(EDirectoryVisitorFlags::ThreadSafe)
		, PlatformFile(InPlatformFile)
		, FoundFiles(InFoundFiles)
		, FileExtension(InFileExtension)
		, FileExtensionLen(InFileExtension ? FCString::Strlen(InFileExtension) : 0)
	{
	}
	virtual bool Visit(const TCHAR* FilenameOrDirectory, bool bIsDirectory) override
	{
		if (!bIsDirectory)
		{
			if (FileExtensionLen > 0)
			{
				int32 FileNameLen = FCString::Strlen(FilenameOrDirectory);
				if (FileNameLen < FileExtensionLen || 
					FCString::Strcmp(&FilenameOrDirectory[FileNameLen - FileExtensionLen], FileExtension) != 0)
				{
					return true;
				}
			}
			
			FString FileName(FilenameOrDirectory);
			FRWScopeLock ScopeLock(FoundFilesLock, SLT_Write);
			FoundFiles.Emplace(MoveTemp(FileName));
		}
		return true;
	}
};

void IPlatformFile::FindFiles(TArray<FString>& FoundFiles, const TCHAR* Directory, const TCHAR* FileExtension)
{
	FFindFilesVisitor FindFilesVisitor(*this, FoundFiles, FileExtension);
	IterateDirectory(Directory, FindFilesVisitor);
}

void IPlatformFile::FindFilesRecursively(TArray<FString>& FoundFiles, const TCHAR* Directory, const TCHAR* FileExtension)
{
	FFindFilesVisitor FindFilesVisitor(*this, FoundFiles, FileExtension);
	IterateDirectoryRecursively(Directory, FindFilesVisitor);
}

bool IPlatformFile::DeleteDirectoryRecursively(const TCHAR* Directory)
{
	class FRecurse : public FDirectoryVisitor
	{
	public:
		IPlatformFile&		PlatformFile;
		uint32 FirstError = 0;

		FRecurse(IPlatformFile&	InPlatformFile)
			: PlatformFile(InPlatformFile)
		{
		}
		virtual bool Visit(const TCHAR* FilenameOrDirectory, bool bIsDirectory)
		{
			if (bIsDirectory)
			{
				PlatformFile.IterateDirectory(FilenameOrDirectory, *this);
				if (!PlatformFile.DeleteDirectory(FilenameOrDirectory) && FirstError == 0)
				{
					FirstError = FPlatformMisc::GetLastError();
				}
			}
			else
			{
				if (PlatformFile.DeleteFile(FilenameOrDirectory))
					return true;

				// File delete failed -- unset readonly flag and try again
				PlatformFile.SetReadOnly(FilenameOrDirectory, false);
				if (!PlatformFile.DeleteFile(FilenameOrDirectory) && FirstError == 0)
				{
					FirstError = FPlatformMisc::GetLastError();
				}
			}
			return true; // continue searching
		}
	};
	FRecurse Recurse(*this);
	Recurse.Visit(Directory, true);
	const bool bSucceeded = !DirectoryExists(Directory);
	if (!bSucceeded)
	{
		FPlatformMisc::SetLastError(Recurse.FirstError);
	}
	return bSucceeded;
}


bool IPlatformFile::CopyFile(const TCHAR* To, const TCHAR* From, EPlatformFileRead ReadFlags, EPlatformFileWrite WriteFlags)
{
	const int64 MaxBufferSize = 1024*1024;

	TUniquePtr<IFileHandle> FromFile(OpenRead(From, (ReadFlags & EPlatformFileRead::AllowWrite) != EPlatformFileRead::None));
	if (!FromFile)
	{
		return false;
	}
	TUniquePtr<IFileHandle> ToFile(OpenWrite(To, false, (WriteFlags & EPlatformFileWrite::AllowRead) != EPlatformFileWrite::None));
	if (!ToFile)
	{
		bool bFailed = true;
		if ((WriteFlags & EPlatformFileWrite::AttemptDeleteAndCreate) != EPlatformFileWrite::None)
		{
			if (DeleteFile(To))
			{
				ToFile = TUniquePtr<IFileHandle>(OpenWrite(To, false, (WriteFlags & EPlatformFileWrite::AllowRead) != EPlatformFileWrite::None));
				bFailed = (!ToFile);
			}
		}
		if (bFailed)
		{
			return false;
		}
	}
	int64 Size = FromFile->Size();
	if (Size < 1)
	{
		check(Size == 0);
		return true;
	}
	int64 AllocSize = FMath::Min<int64>(MaxBufferSize, Size);
	check(AllocSize);
	uint8* Buffer = (uint8*)FMemory::Malloc(int32(AllocSize));
	check(Buffer);
	while (Size)
	{
		int64 ThisSize = FMath::Min<int64>(AllocSize, Size);
		if (!FromFile->Read(Buffer, ThisSize))
		{
			break;
		}
		if (!ToFile->Write(Buffer, ThisSize))
		{
			break;
		}
		Size -= ThisSize;
		check(Size >= 0);
	}
	FMemory::Free(Buffer);
	check(Size >= 0);
	return Size == 0;
}

bool IPlatformFile::CopyDirectoryTree(const TCHAR* DestinationDirectory, const TCHAR* Source, bool bOverwriteAllExisting)
{
	check(DestinationDirectory);
	check(Source);

	FString DestDir(DestinationDirectory);
	FPaths::NormalizeDirectoryName(DestDir);

	FString SourceDir(Source);
	FPaths::NormalizeDirectoryName(SourceDir);

	// Does Source dir exist?
	if (!DirectoryExists(*SourceDir))
	{
		return false;
	}

	// Destination directory exists already or can be created ?
	if (!DirectoryExists(*DestDir) &&
		!CreateDirectoryTree(*DestDir))
	{
		return false;
	}

	// Copy all files and directories
	struct FCopyFilesAndDirs : public FDirectoryVisitor
	{
		IPlatformFile & PlatformFile;
		const TCHAR* SourceRoot;
		const TCHAR* DestRoot;
		bool bOverwrite;

		FCopyFilesAndDirs(IPlatformFile& InPlatformFile, const TCHAR* InSourceRoot, const TCHAR* InDestRoot, bool bInOverwrite)
			: PlatformFile(InPlatformFile)
			, SourceRoot(InSourceRoot)
			, DestRoot(InDestRoot)
			, bOverwrite(bInOverwrite)
		{
		}

		virtual bool Visit(const TCHAR* FilenameOrDirectory, bool bIsDirectory)
		{
			FString NewName(FilenameOrDirectory);
			// change the root
			NewName = NewName.Replace(SourceRoot, DestRoot);

			if (bIsDirectory)
			{
				// create new directory structure
				if (!PlatformFile.CreateDirectoryTree(*NewName) && !PlatformFile.DirectoryExists(*NewName))
				{
					return false;
				}
			}
			else
			{
				// Delete destination file if it exists and we are overwriting
				if (PlatformFile.FileExists(*NewName) && bOverwrite)
				{
					PlatformFile.DeleteFile(*NewName);
				}

				// Copy file from source
				if (!PlatformFile.CopyFile(*NewName, FilenameOrDirectory))
				{
					// Not all files could be copied
					return false;
				}
			}
			return true; // continue searching
		}
	};

	// copy files and directories visitor
	FCopyFilesAndDirs CopyFilesAndDirs(*this, *SourceDir, *DestDir, bOverwriteAllExisting);

	// create all files subdirectories and files in subdirectories!
	return IterateDirectoryRecursively(*SourceDir, CopyFilesAndDirs);
}

FString IPlatformFile::ConvertToAbsolutePathForExternalAppForRead( const TCHAR* Filename )
{
	return FPaths::ConvertRelativePathToFull(Filename);
}

FString IPlatformFile::ConvertToAbsolutePathForExternalAppForWrite( const TCHAR* Filename )
{
	return FPaths::ConvertRelativePathToFull(Filename);
}

static bool InternalCreateDirectoryTree(IPlatformFile& Ipf, const FString& Directory)
{
	// Just try creating directory first
	if (Ipf.CreateDirectory(*Directory))
		return true;

	// If it fails, try creating parent(s), before attempting to create directory
	// once again.
	int32 SeparatorIndex = -1;
	if (Directory.FindLastChar(TEXT('/'), /* out */ SeparatorIndex))
	{
		if (SeparatorIndex)
		{
			if (!InternalCreateDirectoryTree(Ipf, Directory.Left(SeparatorIndex)))
				return false;

			if (Ipf.CreateDirectory(*Directory))
				return true;
		}
	}

	uint32 ErrorCode = FPlatformMisc::GetLastError();
	const bool bExists = Ipf.DirectoryExists(*Directory);
	if (!bExists)
	{
		FPlatformMisc::SetLastError(ErrorCode);
	}
	return bExists;
}

bool IPlatformFile::CreateDirectoryTree(const TCHAR* Directory)
{
	FString LocalDirname(Directory);
	FPaths::NormalizeDirectoryName(LocalDirname);

	return InternalCreateDirectoryTree(*this, LocalDirname);
}

FString FFileJournalFileHandle::ToString()
{
	FString Output;
	TArray<TCHAR, FString::AllocatorType>& CharArray = Output.GetCharArray();
	CharArray.AddUninitialized(sizeof(FFileJournalFileHandle) * 2 + 3);
	TCHAR* Data = CharArray.GetData();
	Data[0] = '0';
	Data[1] = 'x';
	UE::String::BytesToHexLower(Bytes, Data + 2);
	CharArray.Last() = TCHAR('\0');
	return Output;
}

namespace UE::PlatformFileJournal::Private
{

FFileJournalFileHandle CreateInvalidFileHandle()
{
	FFileJournalFileHandle Result;
	for (uint8& Byte : Result.Bytes)
	{
		Byte = 0;
	}
	return Result;
}

FFileJournalData ToJournalData(const FFileStatData& StatData)
{
	FFileJournalData JournalData;
	JournalData.ModificationTime = StatData.ModificationTime;
	JournalData.JournalHandle = FileJournalFileHandleInvalid;
	JournalData.bIsValid = StatData.bIsValid;
	JournalData.bIsDirectory = StatData.bIsDirectory;
	JournalData.bIsReadOnly = StatData.bIsReadOnly;
	return JournalData;
}

constexpr const TCHAR* PlatformNotAvailableMessage = TEXT("PlatformFileJournal is not implemented on the current platform.");

} // namespace UE::PlatformFileJournal::Private

const FFileJournalFileHandle FileJournalFileHandleInvalid = UE::PlatformFileJournal::Private::CreateInvalidFileHandle();

bool IPlatformFile::FileJournalIsAvailable(const TCHAR* VolumeOrPath, ELogVerbosity::Type* OutErrorLevel,
	FString* OutError)
{
	if (OutErrorLevel)
	{
		*OutErrorLevel = ELogVerbosity::Display;
	}
	if (OutError)
	{
		*OutError = UE::PlatformFileJournal::Private::PlatformNotAvailableMessage;
	}
	return false;
}

uint64 IPlatformFile::FileJournalGetMaximumSize(const TCHAR* VolumeOrPath, ELogVerbosity::Type* OutErrorLevel,
	FString* OutError) const
{
	if (OutErrorLevel)
	{
		*OutErrorLevel = ELogVerbosity::Display;
	}
	if (OutError)
	{
		*OutError = UE::PlatformFileJournal::Private::PlatformNotAvailableMessage;
	}
	return 0;
}

EFileJournalResult IPlatformFile::FileJournalGetLatestEntry(const TCHAR* VolumeOrPath,
	FFileJournalId& OutJournalId, FFileJournalEntryHandle& OutEntryHandle, FString* OutError)
{
	if (OutError)
	{
		*OutError = UE::PlatformFileJournal::Private::PlatformNotAvailableMessage;
	}
	OutJournalId = FileJournalIdInvalid;
	OutEntryHandle = FileJournalEntryHandleInvalid;
	return EFileJournalResult::InvalidPlatform;
}

bool IPlatformFile::FileJournalIterateDirectory(const TCHAR* Directory, FDirectoryJournalVisitorFunc Visitor, FString* OutError)
{
	return this->IterateDirectoryStat(Directory,
		[&Visitor](const TCHAR* InPackageFilename, const FFileStatData& StatData)
		{
			return Visitor(InPackageFilename, UE::PlatformFileJournal::Private::ToJournalData(StatData));
		});
}

FFileJournalData IPlatformFile::FileJournalGetFileData(const TCHAR* FilenameOrDirectory, FString* OutError)
{
	return UE::PlatformFileJournal::Private::ToJournalData(this->GetStatData(FilenameOrDirectory));
}

EFileJournalResult IPlatformFile::FileJournalReadModified(const TCHAR* VolumeName,
	const FFileJournalId& JournalIdOfStartingEntry, const FFileJournalEntryHandle& StartingJournalEntry,
	TMap<FFileJournalFileHandle, FString>& KnownDirectories, TSet<FString>& OutModifiedDirectories,
	FFileJournalEntryHandle& OutNextJournalEntry, FString* OutError)
{
	OutNextJournalEntry = FileJournalEntryHandleInvalid;
	if (OutError)
	{
		*OutError = UE::PlatformFileJournal::Private::PlatformNotAvailableMessage;
	}
	return EFileJournalResult::InvalidPlatform;
}

FString IPlatformFile::FileJournalGetVolumeName(FStringView Path)
{
	FString FullPath = FPaths::ConvertRelativePathToFull(FString(Path));
	FStringView VolumeName;
	FStringView Remainder;
	FPathViews::SplitVolumeSpecifier(FullPath, VolumeName, Remainder);
	return FString(VolumeName);
}

bool IPhysicalPlatformFile::Initialize(IPlatformFile* Inner, const TCHAR* CmdLine)
{
	// Physical platform file should never wrap anything.
	check(Inner == NULL);
	return true;
}
