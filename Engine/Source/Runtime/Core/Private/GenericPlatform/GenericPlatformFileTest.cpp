// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFile.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/Paths.h"
#include "Misc/PathViews.h"
#include "Misc/AutomationTest.h"
#include "HAL/PlatformMisc.h"
#include "Templates/UniquePtr.h"

#include "Async/AsyncFileHandle.h"
#include "Async/MappedFileHandle.h"

#if WITH_DEV_AUTOMATION_TESTS

#if WITH_ENGINE

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMMapFileReadTest, "System.Engine.Files.MMapFileRead", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::ClientContext | EAutomationTestFlags::EngineFilter);

bool FMMapFileReadTest::RunTest(const FString& InParameter)
{
	const FString TempDir = FPaths::AutomationTransientDir();
	const FString ReadPrefix = "MMap_FileToRead";
	const FString TxtExtension = ".txt";
	const FString TempFileToRead = FPaths::ConvertRelativePathToFull(FPaths::CreateTempFilename(*TempDir, *ReadPrefix, *TxtExtension));
	const FString TestDirectory = FPaths::GetPath(TempFileToRead);

	// Make sure the directory exists
	const bool bMakeTree = true;
	UTEST_TRUE("Making directory tree", IFileManager::Get().MakeDirectory(*TestDirectory, bMakeTree));

	// Create a dummy file to read later with OpenMappedEx2
	const char FileContent[] = "Temp file to read.";
	IFileHandle* FileHandle = nullptr;
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();

	FileHandle = PlatformFile.OpenWrite(*TempFileToRead);

	if (FileHandle == nullptr)
	{
		return false;
	}

	FileHandle->Write((uint8*)FileContent, sizeof(FileContent));
	FileHandle->Flush();
	delete FileHandle;

	FOpenMappedResult ResMap = FPlatformFileManager::Get().GetPlatformFile().OpenMappedEx2(*TempFileToRead);
	TUniquePtr<IMappedFileHandle> Handle = ResMap.HasError() ? nullptr : ResMap.StealValue();

	if (Handle)
	{
		IMappedFileRegion* Region = Handle->MapRegion();

		int64 Size = Region->GetMappedSize();

		UTEST_TRUE("Mapped size valid", Size >= sizeof(FileContent));

		const char* Data = (const char*)Region->GetMappedPtr();

		const bool bMemcmpSuccess = FMemory::Memcmp(FileContent, Data, sizeof(FileContent)) == 0;
		UTEST_TRUE("Mapped data valid", bMemcmpSuccess);

		delete Region;

		return true;
	}

	// Some platforms might not implement OpenMappedEx so the test should not fail
	if (ResMap.HasError())
	{
		if (ResMap.GetError().GetMessage() == TEXT("OpenMappedEx2 is not implemented on this platform"))
		{
			return true;
		}
	}

	return false;
}


IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMMapFileWriteTest, "System.Engine.Files.MMapFileWrite", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::ClientContext | EAutomationTestFlags::EngineFilter);

bool FMMapFileWriteTest::RunTest(const FString& InParameter)
{
	const FString TempDir = FPaths::AutomationTransientDir();
	const FString WritePrefix = "MMap_FileToWrite";
	const FString TxtExtension = ".txt";
	const FString TempFileToWrite = FPaths::ConvertRelativePathToFull(FPaths::CreateTempFilename(*TempDir, *WritePrefix, *TxtExtension));
	const int64	  MapAddressSpaceSize = 1024;
	const FString TestDirectory = FPaths::GetPath(TempFileToWrite);

	// Make sure the directory exists
	const bool bMakeTree = true;
	UTEST_TRUE("Making directory tree", IFileManager::Get().MakeDirectory(*TestDirectory, bMakeTree))

	const char FileContent[] = "Temp file to write.";

	FOpenMappedResult ResMap = FPlatformFileManager::Get().GetPlatformFile().OpenMappedEx2(
		*TempFileToWrite, IPlatformFile::EOpenMappedFlags::OpenForWrite, MapAddressSpaceSize);

	// Some platforms might not implement OpenMappedEx so the test should not fail
	if (ResMap.HasError())
	{
		const bool bNotImplemented = ResMap.GetError().GetMessage() == TEXT("OpenMappedEx2 is not implemented on this platform");
		UTEST_TRUE("OpenMappedEx2 failed but is implemented for the platform", bNotImplemented);

		return bNotImplemented;
	}

	{
		TUniquePtr<IMappedFileHandle> Handle = ResMap.StealValue();

		IMappedFileRegion* Region = Handle->MapRegion(0, MapAddressSpaceSize, EMappedFileFlags::EFileWritable);

		uint8* MapPtr = (uint8*)Region->GetMappedPtr();
		uint64 MapSize = Region->GetMappedSize();

		check(sizeof(FileContent) <= MapSize);
		FMemory::Memcpy(MapPtr, (uint8*)FileContent, sizeof(FileContent));
		Handle->Flush();

		delete Region;
	}

	ResMap = FPlatformFileManager::Get().GetPlatformFile().OpenMappedEx2(
		*TempFileToWrite, IPlatformFile::EOpenMappedFlags::OpenForWrite | IPlatformFile::EOpenMappedFlags::OpenExisting, MapAddressSpaceSize);

	UTEST_TRUE("OpenMappedEx2 OpenForWrite | OpenExisting succeed", ResMap.HasValue());

	{
		TUniquePtr<IMappedFileHandle> Handle = ResMap.StealValue();

		IMappedFileRegion* Region = Handle->MapRegion();

		int64 Size = Region->GetMappedSize();

		UTEST_TRUE("Mapped size valid", Size >= sizeof(FileContent));

		const char* Data = (const char*)Region->GetMappedPtr();

		const bool bMemcmpSuccess = FMemory::Memcmp(FileContent, Data, sizeof(FileContent)) == 0;
		UTEST_TRUE("Mapped data valid", bMemcmpSuccess);

		delete Region;
	}

	return true;
}


#endif // WITH_ENGINE

#endif // WITH_DEV_AUTOMATION_TESTS



