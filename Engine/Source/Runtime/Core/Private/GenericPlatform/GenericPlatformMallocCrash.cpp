// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenericPlatform/GenericPlatformMallocCrash.h"
#include "Containers/ArrayView.h"
#include "HAL/ConsoleManager.h"
#include "HAL/PlatformProcess.h"
#include "HAL/PlatformTLS.h"
#include "Templates/AlignmentTemplates.h"
#include "HAL/LowLevelMemTracker.h"
#include "ProfilingDebugging/MemoryTrace.h"
#include "Algo/BinarySearch.h"

// Keep a separate value to avoid TLS initialization order issues from calling Get() too early
static bool GIsMallocCrashActive = false;

// Suspending other threads on the first allocator call following a crash may help with debugging by freezing
// the program state and avoiding further manipulation of a bad program state. However, crash handling on some
// platforms requires threads to be able to tear down TLS, which in many cases involves deallocating memory.
// These modes allow threads that are not the one that signaled the crash to skip frees or proceed as normal.
enum EMallocCrashNonCrashedThreadsMode
{
	SuspendAllRequests = 0, // Non-crashed threads suspend on allocation or free
	SuspendAllocationsAndSkipFrees = 1 // Non-crashed threads suspend on allocation and do nothing on free
};

// NOTE: Crashes prior to loading config/hotfixes will abide by the default mode, but this CVar may help recover crashes beyond that point
static int32 GMallocCrashNonCrashedThreadsMode = SuspendAllocationsAndSkipFrees;
static FAutoConsoleVariableRef CVarMallocCrashNonCrashedThreadsMode(
	TEXT("MallocCrash.NonCrashedThreadsMode"),
	GMallocCrashNonCrashedThreadsMode,
	TEXT("How to process requests during a crash from threads other than the original crashed thread. ")
	TEXT("0 = Non-crashed threads suspend on allocation or free. ")
	TEXT("1 = Non-crashed threads suspend on allocation and do nothing on free. "),
	ECVF_Default);

/** Describes a pool. */
struct FPoolDesc
{
	constexpr FPoolDesc(uint32 InSize, const uint32 InNumAllocs)
		: Size(InSize)
		, NumAllocs(InNumAllocs)
	{
	}

	/** Size of the pool. */
	const uint32 Size;

	/** Number of allocations in the pool. */
	const uint32 NumAllocs;
};

#ifdef NUM_ALLOCS_MODIFIER
	#error NUM_ALLOCS_MODIFIER already defined
#endif // NUM_ALLOCS_MODIFIER

#if PLATFORM_UNIX
// Linux crash handling exhausts some of pools sometimes
	#define NUM_ALLOCS_MODIFIER		2
#else
	#define NUM_ALLOCS_MODIFIER		1
#endif // PLATFORM_UNIX

template<uint32 AllocSize>
constexpr FPoolDesc MakePoolDesc(uint32 NumAllocs)
{
	static_assert(IsAligned(AllocSize, FGenericPlatformMallocCrash::REQUIRED_ALIGNMENT),
		"All pool sizes in FGenericPlatformMallocCrash must be aligned to required alignment.");
	return FPoolDesc(AllocSize, NumAllocs * NUM_ALLOCS_MODIFIER);
}

#undef NUM_ALLOCS_MODIFIER


static constexpr FPoolDesc AllPoolDesc[FGenericPlatformMallocCrash::NUM_POOLS] =
{
	MakePoolDesc<64>(224),
	MakePoolDesc<96>(144),
	MakePoolDesc<128>(80),
	MakePoolDesc<192>(560),
	MakePoolDesc<256>(384),
	MakePoolDesc<384>(208),
	MakePoolDesc<512>(48),
	MakePoolDesc<768>(32),
	MakePoolDesc<1024>(32),
	MakePoolDesc<2048>(32),
	MakePoolDesc<4096>(32),
	MakePoolDesc<8192>(32),
	MakePoolDesc<16384>(16),
	MakePoolDesc<32768>(16),
};

/** Generated by the FGenericPlatformMallocCrash::PrintPoolsUsage. */
const FPoolDesc& FGenericPlatformMallocCrash::GetPoolDesc( uint32 Index ) const
{
	return AllPoolDesc[Index];
}

struct FMallocCrashPool
{
	uint32 NumUsed;
	uint32 MaxUsedIndex;
	uint32 MaxNumUsed;
	uint32 TotalNumUsed;

	/** Allocation size for this pool. */
	const uint32 AllocationSize;

	/** Fixed list of allocations for the specified size for this pool. */
	TArrayView<FPtrInfo> Allocations;

	// All allocations are used within this region of memory, use to match allocations to pools when freeing
	uint8* AllocBase;

	/** Maximum number of allocations that can be made for this pool. */
	const uint32 MaxNumAllocations;

	/** Memory allocated in the pool and memory used by a fixed array. */
	uint32 AllocatedMemory;

	FMallocCrashPool( const FPoolDesc& PoolDesc, FGenericPlatformMallocCrash& Outer ): 
		NumUsed(0),
		MaxUsedIndex(0),
		MaxNumUsed(0),
		TotalNumUsed(0),
		AllocationSize( PoolDesc.Size ),
		MaxNumAllocations( PoolDesc.NumAllocs )
	{
		Allocations = TArrayView<FPtrInfo>(
			reinterpret_cast<FPtrInfo*>(Outer.AllocateFromBookkeeping(MaxNumAllocations * sizeof(FPtrInfo))), 
			MaxNumAllocations
			);

		AllocBase = (uint8*)Outer.AllocateFromSmallPool(AllocationSize * MaxNumAllocations);
		for( uint32 Index = 0; Index < MaxNumAllocations; ++Index )
		{
			new (&Allocations[Index]) FPtrInfo(AllocBase + Index * AllocationSize);
		}
		
		AllocatedMemory = MaxNumAllocations*AllocationSize + MaxNumAllocations*sizeof(FPtrInfo);

#ifdef	_DEBUG
		FPlatformMisc::LowLevelOutputDebugStringf( TEXT( "FMallocCrashPool Size=%5u Num=%4i Mem=%8u\n" ), PoolDesc.Size, MaxNumAllocations, AllocatedMemory );
#endif // _DEBUG
	}

	uint8* AllocateFromPool( uint32 InAllocationSize )
	{
		DebugVerify();

		// Find an unused allocation.
		FPtrInfo* PtrInfo = nullptr;
		for (uint32 Index = 0; Index < MaxNumAllocations; ++Index)
		{
			if(FPtrInfo& PtrIt = Allocations[Index]; PtrIt.Size == 0)
			{
				PtrInfo = &PtrIt;
				MaxUsedIndex = FMath::Max(MaxUsedIndex,Index);
				break;
			}
		}

		if( PtrInfo )
		{
			NumUsed++;
			TotalNumUsed++;
			MaxNumUsed = FMath::Max(MaxNumUsed,NumUsed);
			PtrInfo->Size = InAllocationSize;

			FMemory::Memset( (void*)PtrInfo->Ptr, FGenericPlatformMallocCrash::MEM_TAG, PtrInfo->Size );
			DebugVerify();

			//FPlatformMisc::LowLevelOutputDebugStringf( TEXT( "Malloc Requested=%u Size=%u, PooledPtr=0x%016llx \n" ), InAllocationSize, AllocationSize, (uint64)PtrInfo->Ptr );

			return PtrInfo->Ptr;
		}
		else
		{
			UE_DEBUG_BREAK();
			FPlatformMisc::LowLevelOutputDebugStringf( TEXT( "AllocateFromPool run out of memory allocating %u bytes for %u allocations\n" ), InAllocationSize, MaxNumAllocations );
			FPlatformMisc::LowLevelOutputDebugString( TEXT( "Please increase MaxNumAllocations for that pool, exiting...\n" ) );
			FPlatformMisc::RequestExit( true, TEXT("GenericPlatformmallocCrash::AllocateFromPool"));
		}
		return nullptr;
	}

	/** Tries to free a pointer. */
	void TryFreeFromPool( uint8* Ptr )
	{
		//const uint32 PtrSize = FGenericPlatformMallocCrash::GetAllocationSize(Ptr);
		//FPlatformMisc::LowLevelOutputDebugStringf( TEXT( "Free SizeWithOverhead=%u, PooledPtr=0x%016llx \n" ), PtrSize, (uint64)Ptr );

		int32 Index = int32((Ptr - AllocBase) / AllocationSize);
		if (Index < 0 || Index >=(int32)MaxNumAllocations)
		{
			FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Failed to free PoolSize=%u, PooledPtr=0x%016llx \n"),
				AllocationSize, (uint64)Ptr);

			UE_DEBUG_BREAK();
		}
		else
		{
			FPtrInfo& PtrIt = Allocations[Index];
			FMemory::Memset((void*)PtrIt.Ptr, FGenericPlatformMallocCrash::MEM_WIPETAG, PtrIt.Size);

			PtrIt.Size = 0;
			NumUsed--;
		}

		
		DebugVerify();
	}

	uint64 GetAllocationSize(uint8* Ptr) const
	{
		SIZE_T Offset = Ptr - AllocBase;
		int32 Index = int32(Offset / AllocationSize);
		if (Index < 0 || (uint32)Index >= MaxNumAllocations)
		{
			UE_DEBUG_BREAK();
		}
		return Allocations[Index].Size;
	}

	FORCEINLINE static FMallocCrashPool& GetPool(int32 Index)
	{
		checkSlow(Index >= 0 && Index < FGenericPlatformMallocCrash::NUM_POOLS);
		return ((FMallocCrashPool*)&BytePools)[Index];
	}

	FORCEINLINE static TConstArrayView<FMallocCrashPool> GetPools()
	{
		return TConstArrayView<FMallocCrashPool>((const FMallocCrashPool*)&BytePools, FGenericPlatformMallocCrash::NUM_POOLS);
	}

private:
	void DebugVerify()
	{
#ifdef	_DEBUG
		for( uint32 Index = 0; Index < MaxNumAllocations; ++Index )
		{
			if (FPtrInfo& PtrIt = Allocations[Index]; PtrIt.Size > 32768)
			{
				UE_DEBUG_BREAK();
			}
		}
#endif // _DEBUG
	}

	static TTypeCompatibleBytes<FMallocCrashPool[FGenericPlatformMallocCrash::NUM_POOLS]> BytePools;
};

TTypeCompatibleBytes<FMallocCrashPool[FGenericPlatformMallocCrash::NUM_POOLS]> FMallocCrashPool::BytePools;


FGenericPlatformMallocCrash::FGenericPlatformMallocCrash( FMalloc* MainMalloc ) 
	: SmallMemoryPoolSize(CalculateSmallPoolTotalSize())
	, BookkeepingPoolSize(CalculateBookkeepingPoolTotalSize())
	, PreviousMalloc(MainMalloc)
{
	LLM_SCOPE(ELLMTag::GenericPlatformMallocCrash);
	LLM_PLATFORM_SCOPE(ELLMTag::GenericPlatformMallocCrashPlatform);

	const uint32 LargeMemoryPoolSize = Align((int32)LARGE_MEMORYPOOL_SIZE,SafePageSize());
	LargeMemoryPool = (uint8*)FPlatformMemory::BinnedAllocFromOS(LargeMemoryPoolSize);
	SmallMemoryPool = (uint8*)FPlatformMemory::BinnedAllocFromOS((SIZE_T)SmallMemoryPoolSize);
	BookkeepingPool = (uint8*)FPlatformMemory::BinnedAllocFromOS((SIZE_T)BookkeepingPoolSize);

	LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelAlloc(ELLMTracker::Default, LargeMemoryPool, LargeMemoryPoolSize));
	LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelAlloc(ELLMTracker::Default, SmallMemoryPool, SmallMemoryPoolSize));
	LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelAlloc(ELLMTracker::Default, BookkeepingPool, BookkeepingPoolSize));

	MemoryTrace_Alloc((uint64)LargeMemoryPool, LargeMemoryPoolSize, alignof(uint8), EMemoryTraceRootHeap::SystemMemory);
	MemoryTrace_Alloc((uint64)SmallMemoryPool, SmallMemoryPoolSize, alignof(uint8), EMemoryTraceRootHeap::SystemMemory);
	MemoryTrace_Alloc((uint64)BookkeepingPool, BookkeepingPoolSize, alignof(uint8), EMemoryTraceRootHeap::SystemMemory);

	if( !SmallMemoryPool || !LargeMemoryPool || !BookkeepingPool )
	{
		FPlatformMisc::LowLevelOutputDebugString( TEXT( "Memory pools allocations failed, exiting...\n" ) );
		FPlatformMisc::RequestExit(true, TEXT("GenericPlatformmallocCrash().MemoryPoolsAllocationsFailed"));
	}

	if(!IsAligned(LargeMemoryPool, REQUIRED_ALIGNMENT) 
	|| !IsAligned(SmallMemoryPool, REQUIRED_ALIGNMENT) 
	|| !IsAligned(BookkeepingPool, REQUIRED_ALIGNMENT))
	{
		FPlatformMisc::LowLevelOutputDebugString( TEXT( "OS allocations must be aligned to a value multiple of 16, exiting...\n" ) );
		FPlatformMisc::RequestExit(true, TEXT("GenericPlatformmallocCrash().InvalidAlignment"));
	}

	InitializeSmallPools();
#ifdef	_DEBUG
	FPlatformMisc::LowLevelOutputDebugStringf( TEXT( "FGenericPlatformMallocCrash overhead is %u bytes\n" ), LargeMemoryPoolSize+CalculateSmallPoolTotalSize() );
#endif // _DEBUG
}

FGenericPlatformMallocCrash::~FGenericPlatformMallocCrash()
{
	for (int32 Index = NUM_POOLS; Index; )
	{
		--Index;
		DestructItem(&FMallocCrashPool::GetPool(Index));
	}
}

FGenericPlatformMallocCrash& FGenericPlatformMallocCrash::Get( FMalloc* MainMalloc /*= nullptr*/ )
{
	static FGenericPlatformMallocCrash CrashMalloc( MainMalloc );
	return CrashMalloc;
}

void FGenericPlatformMallocCrash::SetAsGMalloc()
{
	InternalLock.Lock();
	FPlatformAtomics::InterlockedExchangePtr((void**)&UE::Private::GMalloc, this);
	if (PLATFORM_USES_FIXED_GMalloc_CLASS && GFixedMallocLocationPtr)
	{
		*GFixedMallocLocationPtr = this; // replace fixed allocator with crash allocator
	}
	CrashedThreadId = FPlatformTLS::GetCurrentThreadId();
	GIsMallocCrashActive = true;
}

bool FGenericPlatformMallocCrash::IsActive()
{
	return GIsMallocCrashActive;
}

void* FGenericPlatformMallocCrash::Malloc( SIZE_T Size, uint32 Alignment )
{
	const uint32 Size32 = (uint32)Size;
	if( Alignment > 16 )
	{
		UE_DEBUG_BREAK();
		FPlatformMisc::LowLevelOutputDebugString( TEXT( "Alignment > 16 is not supported\n" ) );
	}

	if( CheckThreadForAllocation() )
	{
		FMallocCrashPool* Pool = ChoosePoolForSize( Size32 );
		if( Pool )
		{
			const uint8* PooledPtr = Pool->AllocateFromPool( Size32 );	
			return (void*)PooledPtr;
		}
		else
		{
			const uint32 SizeWithOverhead = Size32 + sizeof(FPtrInfo);
			LargeMemoryPoolOffset = Align( LargeMemoryPoolOffset, REQUIRED_ALIGNMENT );

			if( LargeMemoryPoolOffset + SizeWithOverhead <= LARGE_MEMORYPOOL_SIZE )
			{
				const uint32 ReturnMemoryPoolOffset = LargeMemoryPoolOffset;
				LargeMemoryPoolOffset += SizeWithOverhead;

				FPtrInfo* PtrInfo = (FPtrInfo*)(LargeMemoryPool+ReturnMemoryPoolOffset);
				PtrInfo->Size = Size32;
				PtrInfo->Ptr = LargeMemoryPool+ReturnMemoryPoolOffset+sizeof(FPtrInfo);

				FPlatformMisc::LowLevelOutputDebugStringf( TEXT( "Malloc Size=%d LargeMemoryPoolOffset=%d \n" ), Size32, LargeMemoryPoolOffset );
				return (void*)PtrInfo->Ptr;
			}
			else
			{
				FPlatformMisc::LowLevelOutputDebugStringf( TEXT( "MallocCrash run out of memory allocating %u bytes, free %u bytes\n" ), Size32, LARGE_MEMORYPOOL_SIZE-LargeMemoryPoolOffset );
				FPlatformMisc::LowLevelOutputDebugString(TEXT("Please increase LARGE_MEMORYPOOL_SIZE, exiting...\n"));
				UE_DEBUG_BREAK();
				FPlatformMisc::RequestExit( true, TEXT("GenericPlatformmallocCrash::Malloc.OutOfMemory"));
			}
		}
	}
	return nullptr;
}

void* FGenericPlatformMallocCrash::Realloc( void* Ptr, SIZE_T NewSize, uint32 Alignment )
{
	if( CheckThreadForAllocation() )
	{
		void* Result = nullptr;
		if( Ptr && NewSize )
		{
			SIZE_T PtrSize = 0;
			const bool bPreviousMalloc = NewSize > 0 && Ptr && !(IsPtrInLargePool(Ptr)||IsPtrInSmallPool(Ptr));

			if( bPreviousMalloc )
			{
				// We can safely get allocation size only from a few mallocs, this may change in future.
				if( FCStringWide::Stricmp( PreviousMalloc->GetDescriptiveName(), TEXT("binned") ) == 0 ||
					FCStringWide::Stricmp( PreviousMalloc->GetDescriptiveName(), TEXT("binned2")) == 0 ||
					FCStringWide::Stricmp( PreviousMalloc->GetDescriptiveName(), TEXT("binned3")) == 0 ||
					FCStringWide::Stricmp( PreviousMalloc->GetDescriptiveName(), TEXT("jemalloc") ) == 0 )
				{
					// Realloc from the previous allocator.
					if (!PreviousMalloc->GetAllocationSize(Ptr, PtrSize) || PtrSize == 0)
					{
						FPlatformMisc::LowLevelOutputDebugString( TEXT( "Realloc from previous malloc - we were not able to get correct allocation size, exiting...\n" ) );
						FPlatformMisc::RequestExit( true, TEXT("GenericPlatformmallocCrash::Realloc.InvalidAllocationSize"));
					}
				}
				// There is nothing we can do about it.
				else
				{
					FPlatformMisc::LowLevelOutputDebugString( TEXT( "Realloc from previous malloc - we don't know how to get allocation size, exiting...\n" ) );
					FPlatformMisc::RequestExit( true, TEXT("GenericPlatformmallocCrash::Realloc.CanNotHandleMissingAllocationSize"));
				}
			}
			else
			{
				PtrSize = GetAllocationSize(Ptr);
			}
			
			Result = Malloc( NewSize, REQUIRED_ALIGNMENT );
			FMemory::Memcpy( Result, Ptr, FMath::Min(NewSize,PtrSize) );
			
			if( PtrSize > 32768 )
			{
				FPlatformMisc::LowLevelOutputDebugStringf( TEXT( "Realloc PtrSize=%u NewSize=%u PooledPtr=0x%016llx\n" ), (uint32)PtrSize, (uint32)NewSize, (uint64)Ptr );
			}

			Free( Ptr );
		}
		else if( Ptr == nullptr )
		{
			Result = Malloc( NewSize, REQUIRED_ALIGNMENT );
		}
		else
		{
			Free( Ptr );
			Result = nullptr;
		}
		return Result;
	}
	return nullptr;
}

void FGenericPlatformMallocCrash::Free( void* Ptr )
{
	if( CheckThreadForFree() )
	{
		if( IsPtrInSmallPool(Ptr) )
		{
			FMallocCrashPool* Pool = FindPoolForAlloc(Ptr);
			if( Pool )
			{
				Pool->TryFreeFromPool( (uint8*)Ptr );
			}
			else
			{
				UE_DEBUG_BREAK();
			}
		}
		else if( IsPtrInLargePool(Ptr) )
		{
			// Not implemented yet.
		}
		else
		{
			// From the previous allocator.
		}
	}
}

void FGenericPlatformMallocCrash::PrintPoolsUsage()
{
#ifdef	_DEBUG
	FPlatformMisc::LowLevelOutputDebugString( TEXT( "FPoolDesc used\n" ) );
	for( uint32 Index = 0; Index < FGenericPlatformMallocCrash::NUM_POOLS; ++Index )
	{
		const FMallocCrashPool& CrashPool = FMallocCrashPool::GetPool(Index);
		FPlatformMisc::LowLevelOutputDebugStringf( TEXT( "FPoolDesc(%5u,%4u),\n" ), CrashPool.AllocationSize, CrashPool.MaxUsedIndex );
	}

	FPlatformMisc::LowLevelOutputDebugString( TEXT( "FPoolDesc tweaked\n" ) );
	for( uint32 Index = 0; Index < FGenericPlatformMallocCrash::NUM_POOLS; ++Index )
	{
		const FMallocCrashPool& CrashPool = FMallocCrashPool::GetPool(Index);
		FPlatformMisc::LowLevelOutputDebugStringf( TEXT( "FPoolDesc(%5u,%4u),\n" ), CrashPool.AllocationSize, Align(CrashPool.MaxUsedIndex*2+16,16) );
	}
	FPlatformMisc::LowLevelOutputDebugStringf( TEXT( "LargeMemoryPoolOffset=%u\n" ), LargeMemoryPoolOffset );
#endif // _DEBUG
}

PRAGMA_DISABLE_UNREACHABLE_CODE_WARNINGS
bool FGenericPlatformMallocCrash::CheckThreadForAllocation() const
{
	// Crashing thread always processes allocations as normal, other threads abide by the current mode
	if (CrashedThreadId == FPlatformTLS::GetCurrentThreadId())
	{
		return true;
	}
	else
	{
		switch (GMallocCrashNonCrashedThreadsMode)
		{
			case EMallocCrashNonCrashedThreadsMode::SuspendAllocationsAndSkipFrees:
			case EMallocCrashNonCrashedThreadsMode::SuspendAllRequests:
				FPlatformProcess::SleepInfinite();
				return false;

			default:
				return false;
		}
	}
}

bool FGenericPlatformMallocCrash::CheckThreadForFree() const
{
	// Crashing thread always processes frees as normal, other threads abide by the current mode
	if (CrashedThreadId == FPlatformTLS::GetCurrentThreadId())
	{
		return true;
	}
	else
	{
		switch (GMallocCrashNonCrashedThreadsMode)
		{
			case EMallocCrashNonCrashedThreadsMode::SuspendAllRequests:
				FPlatformProcess::SleepInfinite();
				return false;

			case EMallocCrashNonCrashedThreadsMode::SuspendAllocationsAndSkipFrees:
			default:
				return false;
		}
	}
}
PRAGMA_RESTORE_UNREACHABLE_CODE_WARNINGS

bool FGenericPlatformMallocCrash::IsPtrInLargePool( void* Ptr ) const
{
	const bool bResult = (Ptr >= &LargeMemoryPool[0] && Ptr < &LargeMemoryPool[LARGE_MEMORYPOOL_SIZE]);
	return bResult;
}

bool FGenericPlatformMallocCrash::IsPtrInSmallPool( void* Ptr ) const
{
	const bool bResult = (Ptr >= &SmallMemoryPool[0] && Ptr < &SmallMemoryPool[SmallMemoryPoolSize]);
	return bResult;
}

bool FGenericPlatformMallocCrash::GetAllocationSize( void *Original, SIZE_T &SizeOut )
{
	SizeOut = FGenericPlatformMallocCrash::GetAllocationSize(Original);
	return true;
}

uint32 FGenericPlatformMallocCrash::CalculateSmallPoolTotalSize() const
{
	static uint32 TotalSize = 0;
	if( TotalSize == 0 )
	{
		for( uint32 Index = 0; Index < NUM_POOLS; ++Index )
		{
			const FPoolDesc& PoolDesc = GetPoolDesc(Index);
			check(PoolDesc.NumAllocs%16==0);
			check(PoolDesc.Size%16==0);
			TotalSize += PoolDesc.NumAllocs*PoolDesc.Size;
		}

		TotalSize = Align(TotalSize,SafePageSize());
	}
	return TotalSize;
}


uint32 FGenericPlatformMallocCrash::CalculateBookkeepingPoolTotalSize() const
{
	static uint32 TotalSize = 0;
	if (TotalSize == 0)
	{
		for (uint32 Index = 0; Index < NUM_POOLS; ++Index)
		{
			const FPoolDesc& PoolDesc = GetPoolDesc(Index);
			check(PoolDesc.NumAllocs % 16 == 0);
			check(PoolDesc.Size % 16 == 0);
			TotalSize += PoolDesc.NumAllocs * sizeof(FPtrInfo);
		}

		TotalSize = Align(TotalSize, SafePageSize());
	}
	return TotalSize;
}

void FGenericPlatformMallocCrash::InitializeSmallPools()
{
	for( uint32 Index = 0; Index < NUM_POOLS; ++Index )
	{
		FMallocCrashPool* NewPool = new (&FMallocCrashPool::GetPool(Index)) FMallocCrashPool(GetPoolDesc(Index), *this);
	}

	check(SmallMemoryPoolOffset<=CalculateSmallPoolTotalSize());
}

FMallocCrashPool* FGenericPlatformMallocCrash::ChoosePoolForSize( uint32 AllocationSize ) const
{
	for( uint32 Index = 0; Index < NUM_POOLS; ++Index )
	{
		FMallocCrashPool* Pool = &FMallocCrashPool::GetPool(Index);

		// If we have used up this pools allocation limit lets try the next one
		if (Pool->NumUsed >= Pool->MaxNumAllocations)
		{
			continue;
		}

		if (AllocationSize <= Pool->AllocationSize)
		{
			return Pool;
		}
	}

	// Use large allocation pool.
	return nullptr;
}

FMallocCrashPool* FGenericPlatformMallocCrash::FindPoolForAlloc(void* Ptr) const
{
	TArrayView<const FMallocCrashPool> Pools = FMallocCrashPool::GetPools();
	int32 IndexBeyond = Algo::UpperBoundBy(Pools, (uint8*)Ptr, [](const FMallocCrashPool& Pool) { return Pool.AllocBase; });
	return &FMallocCrashPool::GetPool(IndexBeyond - 1);
}

uint8* FGenericPlatformMallocCrash::AllocateFromSmallPool( uint32 AllocationSize )
{
	if( SmallMemoryPoolOffset + AllocationSize <= SmallMemoryPoolSize )
	{
		const uint32 ReturnMemoryPoolOffset = SmallMemoryPoolOffset;
		SmallMemoryPoolOffset += AllocationSize;
		return (uint8*)SmallMemoryPool+ReturnMemoryPoolOffset;
	}

	check(0);
	return nullptr;
}

uint8* FGenericPlatformMallocCrash::AllocateFromBookkeeping(uint32 AllocationSize)
{
	if (BookkeepingPoolOffset + AllocationSize <= BookkeepingPoolSize)
	{
		const uint32 ReturnMemoryPoolOffset = BookkeepingPoolOffset;
		BookkeepingPoolOffset += AllocationSize;
		return (uint8*)BookkeepingPool + ReturnMemoryPoolOffset;
	}

	check(0);
	return nullptr;
}

uint64 FGenericPlatformMallocCrash::GetAllocationSize(void *Original)
{
	if (IsPtrInSmallPool(Original))
	{
		FMallocCrashPool* Pool = FindPoolForAlloc(Original);
		return Pool->GetAllocationSize((uint8*)Original);
	}
	else if (IsPtrInLargePool(Original))
	{
		FPtrInfo* Info = (FPtrInfo*)((uint8*)Original - sizeof(FPtrInfo));
		return Info->Size;
	}
	
	return 0;
}

uint32 FGenericPlatformMallocCrash::SafePageSize()
{
	const uint32 PageSize = (uint32)FPlatformMemory::GetStats().PageSize;
	if( PageSize == 0 )
	{
		return 65536u;
	}
	return PageSize;
}







FGenericStackBasedMallocCrash::FGenericStackBasedMallocCrash(FMalloc* MainMalloc)
{
	CurrentFreeMemPtr = (uint8*)FMemory::Malloc(MEMORYPOOL_SIZE);
	FreeMemoryEndPtr = CurrentFreeMemPtr + MEMORYPOOL_SIZE;
}

FGenericStackBasedMallocCrash::~FGenericStackBasedMallocCrash()
{
}

FGenericStackBasedMallocCrash& FGenericStackBasedMallocCrash::Get(FMalloc* MainMalloc /*= nullptr*/)
{
	static FGenericStackBasedMallocCrash CrashMalloc(MainMalloc);
	return CrashMalloc;
}

void FGenericStackBasedMallocCrash::SetAsGMalloc()
{
	if (PLATFORM_USES_FIXED_GMalloc_CLASS && GFixedMallocLocationPtr)
	{
		*GFixedMallocLocationPtr = this; // replace fixed allocator with crash allocator
	}
	UE::Private::GMalloc = this;
	GIsMallocCrashActive = true;
}

bool FGenericStackBasedMallocCrash::IsActive()
{
	return GIsMallocCrashActive;
}

void* FGenericStackBasedMallocCrash::Malloc(SIZE_T Size, uint32 Alignment)
{
	Alignment = FMath::Max(Size >= 16 ? (uint32)16 : (uint32)8, Alignment);

	SIZE_T TotalSize = Size + Alignment + sizeof(SIZE_T);
	void* Ptr = CurrentFreeMemPtr;
	check(Ptr);
	uint8* NewFreeMemPtr = (uint8*)Ptr + TotalSize;
	if (NewFreeMemPtr <= FreeMemoryEndPtr)
	{
		void* Result = Align((uint8*)Ptr + sizeof(SIZE_T), Alignment);
		*((SIZE_T*)((uint8*)Result - sizeof(SIZE_T))) = Size;
		CurrentFreeMemPtr = NewFreeMemPtr;
		return Result;
	}
	check(false)
		return nullptr;
}

void* FGenericStackBasedMallocCrash::Realloc(void* Ptr, SIZE_T NewSize, uint32 Alignment)
{
	if (Ptr && NewSize)
	{
		SIZE_T PtrSize = *((SIZE_T*)((uint8*)Ptr - sizeof(SIZE_T)));
		if (PtrSize == NewSize)
		{
			return Ptr;
		}
		void* Result = Malloc(NewSize, Alignment);
		FMemory::Memcpy(Result, Ptr, FMath::Min(NewSize, PtrSize));
		return Result;
	}
	else
	{
		return Malloc(NewSize, Alignment);
	}
}

void FGenericStackBasedMallocCrash::Free(void* /*Ptr*/)
{
}
