// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Fundamental/Oversubscription.h"
#include "HAL/PlatformMath.h"
#include "Misc/MonotonicTime.h"

#include <atomic>
#include <chrono>
#include <condition_variable>
#include <mutex>

namespace UE::HAL::Private
{

/**
 * A manual reset event that supports only one thread waiting and one thread notifying at a time.
 *
 * Only one waiting thread may call Reset() or the Wait() functions.
 * Only one notifying thread may call Notify() once until the event is reset.
 */
class FGenericPlatformManualResetEvent
{
public:
	FGenericPlatformManualResetEvent() = default;
	FGenericPlatformManualResetEvent(const FGenericPlatformManualResetEvent&) = delete;
	FGenericPlatformManualResetEvent& operator=(const FGenericPlatformManualResetEvent&) = delete;

	/**
	 * Resets the event to permit another Wait/Notify cycle.
	 *
	 * Must only be called by the waiting thread, and only when there is no possibility of waking
	 * occurring concurrently with the reset.
	 */
	void Reset()
	{
		std::unique_lock SelfLock(Lock);
		bWait = true;
	}

	/**
	 * Polls whether the event is in the notified state.
	 *
	 * @return True if notified, otherwise false.
	 */
	bool Poll()
	{
		std::unique_lock SelfLock(Lock);
		return !bWait;
	}

	/**
	 * Waits for Notify() to be called.
	 *
	 * Notify() may be called prior to Wait(), and this will return immediately in that case.
	 */
	void Wait()
	{
		LowLevelTasks::FOversubscriptionScope _;
		std::unique_lock SelfLock(Lock);
		Condition.wait(SelfLock, [this] { return !bWait; });
	}

	/**
	 * Waits for the wait time for Notify() to be called.
	 *
	 * Notify() may be called prior to WaitFor(), and this will return immediately in that case.
	 *
	 * @param WaitTime   Relative time after which waiting is canceled and the thread wakes.
	 * @return True if Notify() was called before the wait time elapsed, otherwise false.
	 */
	bool WaitFor(FMonotonicTimeSpan WaitTime)
	{
		if (WaitTime.IsInfinity())
		{
			Wait();
			return true;
		}

		if (WaitTime <= FMonotonicTimeSpan::Zero())
		{
			std::unique_lock SelfLock(Lock);
			return !bWait;
		}

		LowLevelTasks::FOversubscriptionScope _;
		std::unique_lock SelfLock(Lock);
		const int64 WaitMs = FPlatformMath::CeilToInt64(WaitTime.ToMilliseconds());
		return Condition.wait_for(SelfLock, std::chrono::milliseconds(WaitMs), [this] { return !bWait; });
	}

	/**
	 * Waits until the wait time for Notify() to be called.
	 *
	 * Notify() may be called prior to WaitUntil(), and this will return immediately in that case.
	 *
	 * @param WaitTime   Absolute time after which waiting is canceled and the thread wakes.
	 * @return True if Notify() was called before the wait time elapsed, otherwise false.
	 */
	bool WaitUntil(FMonotonicTimePoint WaitTime)
	{
		return WaitFor(WaitTime - FMonotonicTimePoint::Now());
	}

	/**
	 * Notifies the waiting thread.
	 *
	 * Notify() may be called prior to one of the wait functions, and the eventual wait call will
	 * return immediately when that occurs.
	 */
	void Notify()
	{
		// The lock must be held by Notify() until it is finished accessing its members, otherwise a waiting thread
		// may destroy the event after its wait but before Notify() has finished.
		std::unique_lock SelfLock(Lock);
		bWait = false;
		Condition.notify_one();
	}

private:
	std::mutex Lock;
	std::condition_variable Condition;
	bool bWait = true;
};

} // UE::HAL::Private
