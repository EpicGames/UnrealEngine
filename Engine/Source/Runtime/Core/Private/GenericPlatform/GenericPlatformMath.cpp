// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenericPlatform/GenericPlatformMath.h"
#include "HAL/PlatformMath.h"
#include "Misc/AssertionMacros.h"
#include "Math/UnrealMathUtility.h"
#include "Math/BigInt.h"

static int32 GSRandSeed;


void FGenericPlatformMath::SRandInit( int32 Seed ) 
{
	GSRandSeed = Seed; 
}

int32 FGenericPlatformMath::GetRandSeed()
{
	return GSRandSeed;
}

float FGenericPlatformMath::SRand() 
{ 
	GSRandSeed = (GSRandSeed * 196314165) + 907633515;
	union { float f; int32 i; } Result;
	union { float f; int32 i; } Temp;
	const float SRandTemp = 1.0f;
	Temp.f = SRandTemp;
	Result.i = (Temp.i & 0xff800000) | (GSRandSeed & 0x007fffff);
	return FPlatformMath::Fractional( Result.f );
} 

float FGenericPlatformMath::Atan2(float Y, float X)
{
	//return atan2f(Y,X);
	// atan2f occasionally returns NaN with perfectly valid input (possibly due to a compiler or library bug).
	// We are replacing it with a minimax approximation with a max relative error of 7.15255737e-007 compared to the C library function.
	// On PC this has been measured to be 2x faster than the std C version.

	const float absX = FMath::Abs(X);
	const float absY = FMath::Abs(Y);
	const bool yAbsBigger = (absY > absX);
	float t0 = yAbsBigger ? absY : absX; // Max(absY, absX)
	float t1 = yAbsBigger ? absX : absY; // Min(absX, absY)
	
	if (t0 == 0.f)
		return 0.f;

	float t3 = t1 / t0;
	float t4 = t3 * t3;

	static const float c[7] = {
		+7.2128853633444123e-03f,
		-3.5059680836411644e-02f,
		+8.1675882859940430e-02f,
		-1.3374657325451267e-01f,
		+1.9856563505717162e-01f,
		-3.3324998579202170e-01f,
		+1.0f
	};

	t0 = c[0];
	t0 = t0 * t4 + c[1];
	t0 = t0 * t4 + c[2];
	t0 = t0 * t4 + c[3];
	t0 = t0 * t4 + c[4];
	t0 = t0 * t4 + c[5];
	t0 = t0 * t4 + c[6];
	t3 = t0 * t3;

	t3 = yAbsBigger ? (0.5f * UE_PI) - t3 : t3;
	t3 = (X < 0.0f) ? UE_PI - t3 : t3;
	t3 = (Y < 0.0f) ? -t3 : t3;

	return t3;
}

double FGenericPlatformMath::Atan2(double Y, double X)
{
	if (X == 0.0 && Y == 0.0)
	{
		return 0.0;
	}

	return atan2(Y,X);
}

float FGenericPlatformMath::Fmod(float X, float Y)
{
	const float AbsY = FMath::Abs(Y);
	if (AbsY <= UE_SMALL_NUMBER) // Note: this constant should match that used by VectorMod() implementations
	{
		FmodReportError(X, Y);
		return 0.0;
	}

	return fmodf(X, Y);
}

double FGenericPlatformMath::Fmod(double X, double Y)
{
	const double AbsY = FMath::Abs(Y);
	if (AbsY <= UE_DOUBLE_SMALL_NUMBER) // Note: this constant should match that used by VectorMod() implementations
	{
		FmodReportError(X, Y);
		return 0.0;
	}

	return fmod(X, Y);
}

void FGenericPlatformMath::FmodReportError(float X, float Y)
{
	if (Y == 0)
	{
		ensureMsgf(Y != 0, TEXT("FMath::FMod(X=%f, Y=%f) : Y is zero, this is invalid and would result in NaN!"), X, Y);
	}
}

void FGenericPlatformMath::FmodReportError(double X, double Y)
{
	if (Y == 0)
	{
		ensureMsgf(Y != 0, TEXT("FMath::FMod(X=%f, Y=%f) : Y is zero, this is invalid and would result in NaN!"), X, Y);
	}
}

#if WITH_DEV_AUTOMATION_TESTS

// Various constants used by the test code below. These are declared
// volatile to prevent compiler optimizations from removing the checks.
namespace CompilerHiddenConstants
{
	volatile float MinusOne = -1.0f;
	volatile float Zero = 0.0f;
	volatile float One = 1.0f;
	volatile float Two = 2.0f;
	volatile float Twelve = 12.0f;
	volatile float Sixteen = 16.0f;
	volatile float MinusOneE37 = -1.0e37f;
	volatile float FloatMax = UE_MAX_FLT;
}

template<class MathPlatform>
class FPlatformMathTest
{
public:
	// Tests for functions that should be implemented in FGenericPlatformMath and can have a platform specific implementation. 
	static void AutoTest()
	{
		using namespace CompilerHiddenConstants;

		check(MathPlatform::IsNaN(sqrtf(MinusOne)));
		check(!MathPlatform::IsFinite(sqrtf(MinusOne)));
		check(!MathPlatform::IsFinite(-1.0f / Zero));
		check(!MathPlatform::IsFinite(1.0f / Zero));
		check(!MathPlatform::IsNaN(-1.0f / Zero));
		check(!MathPlatform::IsNaN(1.0f / Zero));
		check(!MathPlatform::IsNaN(FloatMax));
		check(MathPlatform::IsFinite(FloatMax));
		check(!MathPlatform::IsNaN(Zero));
		check(MathPlatform::IsFinite(Zero));
		check(!MathPlatform::IsNaN(One));
		check(MathPlatform::IsFinite(One));
		check(!MathPlatform::IsNaN(MinusOneE37));
		check(MathPlatform::IsFinite(MinusOneE37));
		check(MathPlatform::FloorLog2((uint32)Zero) == 0);
		check(MathPlatform::FloorLog2((uint32)One) == 0);
		check(MathPlatform::FloorLog2((uint32)Two) == 1);
		check(MathPlatform::FloorLog2((uint32)Twelve) == 3);
		check(MathPlatform::FloorLog2((uint32)Sixteen) == 4);
	}
};

void FGenericPlatformMath::AutoTest() 
{
	{
		FPlatformMathTest<FPlatformMath>::AutoTest();
		FPlatformMathTest<FGenericPlatformMath>::AutoTest();
	}

	{
		// Shift test
		const uint32 ShiftValue[] = { 0xCACACAC2U, 0x1U, 0x0U, 0x0U, 0x0U, 0x0U, 0x0U, 0x0U };
		int256 TestValue(ShiftValue);
		int256 Shift(TestValue);
		Shift <<= 88;
		Shift >>= 88;
		check(Shift == TestValue);
	}

	{
		int256 Dividend(3806401LL);
		int256 Divisor(3233LL);
		int256 Remainder;
		Dividend.DivideWithRemainder(Divisor, Remainder);
		check(Dividend.ToInt() == 1177LL);
		check(Remainder.ToInt() == 1160LL);	
	}

	{
		// Division test: 4294967296LL / 897LL = 4788146LL, R = 334LL
		int256 Dividend(4294967296LL);
		int256 Divisor(897LL);
		int256 Remainder;
		Dividend.DivideWithRemainder(Divisor, Remainder);
		check(Dividend.ToInt() == 4788146LL);
		check(Remainder.ToInt() == 334LL);
	}

	{
		// Shift test with multiple of 32
		int256 Value(1);
		Value <<= 32;
		check(Value.ToInt() == 4294967296LL);
		Value >>= 32;
		check(Value.ToInt() == 1LL);
	}
}

#endif //WITH_DEV_AUTOMATION_TESTS
