// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenericPlatform/GenericPlatformMemory.h"
#include "HAL/LowLevelMemTracker.h"
#include "HAL/PlatformMemory.h"
#include "Misc/AssertionMacros.h"
#include "Math/UnrealMathUtility.h"
#include "Containers/StringConv.h"
#include "UObject/NameTypes.h"
#include "Logging/LogMacros.h"
#include "Stats/Stats.h"
#include "Containers/Ticker.h"
#include "Misc/FeedbackContext.h"
#include "Async/Async.h"
#include "Async/Mutex.h"
#include "HAL/MallocAnsi.h"
#include "GenericPlatform/GenericPlatformMemoryPoolStats.h"
#include "HAL/MemoryMisc.h"
#include "Misc/CoreDelegates.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CommandLine.h"
#include "Misc/MessageDialog.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Templates/UnrealTemplate.h"
#include "HAL/IConsoleManager.h"
#include "ProfilingDebugging/MemoryTrace.h"
#include "ProfilingDebugging/CountersTrace.h"

DEFINE_STAT(MCR_Physical);
DEFINE_STAT(MCR_PhysicalLLM);
DEFINE_STAT(MCR_GPU);
DEFINE_STAT(MCR_TexturePool);
DEFINE_STAT(MCR_StreamingPool);
DEFINE_STAT(MCR_UsedStreamingPool);

DEFINE_STAT(STAT_TotalPhysical);
DEFINE_STAT(STAT_TotalVirtual);
DEFINE_STAT(STAT_PageSize);
DEFINE_STAT(STAT_TotalPhysicalGB);

DEFINE_STAT(STAT_AvailablePhysical);
DEFINE_STAT(STAT_AvailableVirtual);
DEFINE_STAT(STAT_UsedPhysical);
DEFINE_STAT(STAT_PeakUsedPhysical);
DEFINE_STAT(STAT_UsedVirtual);
DEFINE_STAT(STAT_PeakUsedVirtual);

namespace GenericPlatformMemory
{
	static int32 GLogPlatformMemoryStats = 1;
	static FAutoConsoleVariableRef CVarLogPlatformMemoryStats(
		TEXT("memory.logGenericPlatformMemoryStats"),
		GLogPlatformMemoryStats,
		TEXT("Report Platform Memory Stats)\n"),
		ECVF_Default);

	static float GMemoryPressureCriticalThresholdMB = 0;
	static FAutoConsoleVariableRef CVarMemoryPressureCriticalThresholdMB(
		TEXT("memory.MemoryPressureCriticalThresholdMB"),
		GMemoryPressureCriticalThresholdMB,
		TEXT("When the available physical memory drops below this threshold memory stats will consider this to be at critical pressure.\n"
			"Where a platform can specifically state it's memory pressure this test maybe ignored.\n"
			"0 (default) critical pressure will not use the threshold."),
		ECVF_Default);

	static float GMemoryPressureCriticalThresholdPercent = 0;
	static FAutoConsoleVariableRef CVarMemoryPressureCriticalThresholdPercent(
		TEXT("memory.MemoryPressureCriticalThresholdPercent"),
		GMemoryPressureCriticalThresholdPercent,
		TEXT("When the available physical memory drops below this percentage of total physical memory, memory stats will consider this to be at critical pressure.\n"
			"E.g. 10 means Critical when less than 10% of physical RAM is available.\n"
			"Where a platform can specifically state its memory pressure this test may be ignored.\n"
			"0 (default) critical pressure will not use the threshold."),
		ECVF_Default);
}

#if UE_CHECK_LARGE_ALLOCATIONS
namespace UE::Memory::Private
{
	// this is a console variable ref rather than auto console variable as it's used before the
	// console manager has been set up
	bool GEnableLargeAllocationChecks = false;
	FAutoConsoleVariableRef CVarEnableLargeAllocationChecks(
		TEXT("memory.EnableLargeAllocationChecks"),
		GEnableLargeAllocationChecks,
		TEXT("Turn on ensure which checks no single allocation is greater than 'LargeAllocationThreshold'"),
		ECVF_Default);
	int32 GLargeAllocationThreshold = 1 * 1024 * 1024 * 1024;
	FAutoConsoleVariableRef CVarLargeAllocationThreshold(
		TEXT("memory.LargeAllocationThreshold"),
		GLargeAllocationThreshold,
		TEXT("Maximum size a single allocation can be before setting off the ensure enabled by 'EnableLargeAllocationChecks'"),
		ECVF_Default);
}
#endif

TRACE_DECLARE_MEMORY_COUNTER(PlatformMemoryTotalPhysical, TEXT("PlatformMemory/TotalPhysical"));
TRACE_DECLARE_MEMORY_COUNTER(PlatformMemoryTotalVirtual, TEXT("PlatformMemory/TotalVirtual"));
TRACE_DECLARE_MEMORY_COUNTER(PlatformMemoryPageSize, TEXT("PlatformMemory/PageSize"));
TRACE_DECLARE_MEMORY_COUNTER(PlatformMemoryAvailablePhysical, TEXT("PlatformMemory/AvailablePhysical"));
TRACE_DECLARE_MEMORY_COUNTER(PlatformMemoryAvailableVirtual, TEXT("PlatformMemory/AvailableVirtual"));
TRACE_DECLARE_MEMORY_COUNTER(PlatformMemoryUsedPhysical, TEXT("PlatformMemory/UsedPhysical"));
TRACE_DECLARE_MEMORY_COUNTER(PlatformMemoryPeakUsedPhysical, TEXT("PlatformMemory/PeakUsedPhysical"));
TRACE_DECLARE_MEMORY_COUNTER(PlatformMemoryUsedVirtual, TEXT("PlatformMemory/UsedVirtual"));
TRACE_DECLARE_MEMORY_COUNTER(PlatformMemoryPeakUsedVirtual, TEXT("PlatformMemory/PeakUsedVirtual"));

/** Helper class used to update platform memory stats. */
struct FGenericStatsUpdater
{
	/** Called once per second, enqueues stats update. */
	static bool EnqueueUpdateStats( float /*InDeltaTime*/ )
	{
		AsyncTask( ENamedThreads::AnyBackgroundThreadNormalTask, []()
		{
			DoUpdateStats();
		} );
		return true; // Tick again
	}

	/** Gathers and sets all platform memory statistics into the corresponding stats. */
	static void DoUpdateStats()
	{
		static UE::FMutex Mutex;
		// Avoid race in case the ticker runs faster than the background tasks
		// and just skip the update if another one is already running so that
		// we can keep up with the ticker.
		if (Mutex.TryLock())
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FGenericStatsUpdater_DoUpdateStats);

			// This is slow, so do it on the task graph.
			FPlatformMemoryStats MemoryStats = FPlatformMemory::GetStats();
			SET_MEMORY_STAT(STAT_TotalPhysical, MemoryStats.TotalPhysical);
			SET_MEMORY_STAT(STAT_TotalVirtual, MemoryStats.TotalVirtual);
			SET_MEMORY_STAT(STAT_PageSize, MemoryStats.PageSize);
			SET_MEMORY_STAT(STAT_TotalPhysicalGB, MemoryStats.TotalPhysicalGB);

			SET_MEMORY_STAT(STAT_AvailablePhysical, MemoryStats.AvailablePhysical);
			SET_MEMORY_STAT(STAT_AvailableVirtual, MemoryStats.AvailableVirtual);
			SET_MEMORY_STAT(STAT_UsedPhysical, MemoryStats.UsedPhysical);
			SET_MEMORY_STAT(STAT_PeakUsedPhysical, MemoryStats.PeakUsedPhysical);
			SET_MEMORY_STAT(STAT_UsedVirtual, MemoryStats.UsedVirtual);
			SET_MEMORY_STAT(STAT_PeakUsedVirtual, MemoryStats.PeakUsedVirtual);

			TRACE_COUNTER_SET(PlatformMemoryTotalPhysical, MemoryStats.TotalPhysical);
			TRACE_COUNTER_SET(PlatformMemoryTotalVirtual, MemoryStats.TotalVirtual);
			TRACE_COUNTER_SET(PlatformMemoryPageSize, MemoryStats.PageSize);
			TRACE_COUNTER_SET(PlatformMemoryAvailablePhysical, MemoryStats.AvailablePhysical);
			TRACE_COUNTER_SET(PlatformMemoryAvailableVirtual, MemoryStats.AvailableVirtual);
			TRACE_COUNTER_SET(PlatformMemoryUsedPhysical, MemoryStats.UsedPhysical);
			TRACE_COUNTER_SET(PlatformMemoryPeakUsedPhysical, MemoryStats.PeakUsedPhysical);
			TRACE_COUNTER_SET(PlatformMemoryUsedVirtual, MemoryStats.UsedVirtual);
			TRACE_COUNTER_SET(PlatformMemoryPeakUsedVirtual, MemoryStats.PeakUsedVirtual);

			// Platform specific stats.
			FPlatformMemory::InternalUpdateStats(MemoryStats);

			Mutex.Unlock();
		}
	}
};

FGenericPlatformMemoryStats::FGenericPlatformMemoryStats()
	: FGenericPlatformMemoryConstants( FPlatformMemory::GetConstants() )
	, AvailablePhysical( 0 )
	, AvailableVirtual( 0 )
	, UsedPhysical( 0 )
	, PeakUsedPhysical( 0 )
	, UsedVirtual( 0 )
	, PeakUsedVirtual( 0 )
{}

FGenericPlatformMemoryStats::EMemoryPressureStatus FGenericPlatformMemoryStats::GetMemoryPressureStatus() const
{
	const bool bHasMBThreshold      = GenericPlatformMemory::GMemoryPressureCriticalThresholdMB > 0;
	const bool bHasPercentThreshold = GenericPlatformMemory::GMemoryPressureCriticalThresholdPercent > 0;

	if (!bHasMBThreshold && !bHasPercentThreshold)
	{
		return EMemoryPressureStatus::Unknown;
	}

	if (bHasMBThreshold)
	{
		const float AvailablePhysicalMB = float(AvailablePhysical / 1024ull / 1024ull);
		if (AvailablePhysicalMB < GenericPlatformMemory::GMemoryPressureCriticalThresholdMB)
		{
			return EMemoryPressureStatus::Critical;
		}
	}

	if (bHasPercentThreshold && TotalPhysical > 0)
	{
		const float AvailablePhysicalPercent = float(AvailablePhysical) / float(TotalPhysical) * 100.0f;
		if (AvailablePhysicalPercent < GenericPlatformMemory::GMemoryPressureCriticalThresholdPercent)
		{
			return EMemoryPressureStatus::Critical;
		}
	}

	return EMemoryPressureStatus::Nominal;
}

bool FGenericPlatformMemory::bIsOOM = false;
uint64 FGenericPlatformMemory::OOMAllocationSize = 0;
uint32 FGenericPlatformMemory::OOMAllocationAlignment = 0;
FGenericPlatformMemory::EMemoryAllocatorToUse FGenericPlatformMemory::AllocatorToUse = Platform;
void* FGenericPlatformMemory::BackupOOMMemoryPool = nullptr;
uint64 FGenericPlatformMemory::ProgramSize = 0;

void FGenericPlatformMemory::SetupMemoryPools()
{
	SET_MEMORY_STAT(MCR_Physical, 0); // "unlimited" physical memory for the CPU, we still need to make this call to set the short name, etc
	SET_MEMORY_STAT(MCR_PhysicalLLM, 0); // total "unlimited" physical memory, we still need to make this call to set the short name, etc
	SET_MEMORY_STAT(MCR_GPU, 0); // "unlimited" GPU memory, we still need to make this call to set the short name, etc
	SET_MEMORY_STAT(MCR_TexturePool, 0); // "unlimited" Texture memory, we still need to make this call to set the short name, etc
	SET_MEMORY_STAT(MCR_StreamingPool, 0);
	SET_MEMORY_STAT(MCR_UsedStreamingPool, 0);

	// if the platform chooses to have a BackupOOM pool, create it now
	if (FPlatformMemory::GetBackMemoryPoolSize() > 0)
	{
		LLM_PLATFORM_SCOPE(ELLMTag::BackupOOMMemoryPoolPlatform);
		LLM_SCOPE(ELLMTag::BackupOOMMemoryPool);

		BackupOOMMemoryPool = FPlatformMemory::BinnedAllocFromOS(FPlatformMemory::GetBackMemoryPoolSize());

		MemoryTrace_Alloc((uint64)BackupOOMMemoryPool, FPlatformMemory::GetBackMemoryPoolSize(), alignof(void*), EMemoryTraceRootHeap::SystemMemory);
		LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelAlloc(ELLMTracker::Default, BackupOOMMemoryPool, FPlatformMemory::GetBackMemoryPoolSize()));
	}
}

void FGenericPlatformMemory::Init()
{
		SetupMemoryPools();

#if	STATS
	// Stats are updated only once per second.
	const float PollingInterval = 1.0f;
	FTSTicker::GetCoreTicker().AddTicker( FTickerDelegate::CreateStatic( &FGenericStatsUpdater::EnqueueUpdateStats ), PollingInterval );

	// Update for the first time.
	FGenericStatsUpdater::DoUpdateStats();
#endif // STATS
}

void FGenericPlatformMemory::OnOutOfMemory(uint64 Size, uint32 Alignment)
{
	auto HandleOOM = [&]()
	{
		if (GLog)
		{
			GLog->Panic();
		}
			
		// Update memory stats before we enter the crash handler.
		OOMAllocationSize = Size;
		OOMAllocationAlignment = Alignment;

		bIsOOM = true;

		const int ErrorMsgSize = 256;
		TCHAR ErrorMsg[ErrorMsgSize];
		FPlatformMisc::GetSystemErrorMessage(ErrorMsg, ErrorMsgSize, 0);

		FPlatformMemoryStats PlatformMemoryStats = FPlatformMemory::GetStats();
		if (BackupOOMMemoryPool)
		{
			const uint32 BackupPoolSize = FPlatformMemory::GetBackMemoryPoolSize();
			FPlatformMemory::BinnedFreeToOS(BackupOOMMemoryPool, BackupPoolSize);
			UE_LOGF(LogMemory, Warning, "Freeing %d bytes (%.1f MiB) from backup pool to handle out of memory.", BackupPoolSize, double(BackupPoolSize) / (1024 * 1024));
        
			LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelFree(ELLMTracker::Default, BackupOOMMemoryPool));
			MemoryTrace_Free((uint64)BackupOOMMemoryPool, EMemoryTraceRootHeap::SystemMemory);
		}

		UE_LOGF(LogMemory, Warning, "MemoryStats:"
			"\n\tAvailablePhysical %llu (%.2f GiB)"
			"\n\t AvailableVirtual %llu (%.2f GiB)"
			"\n\t     UsedPhysical %llu (%.2f GiB)"
			"\n\t PeakUsedPhysical %llu (%.2f GiB)"
			"\n\t      UsedVirtual %llu (%.2f GiB)"
			"\n\t  PeakUsedVirtual %llu (%.2f GiB)",
			(uint64)PlatformMemoryStats.AvailablePhysical, double(PlatformMemoryStats.AvailablePhysical) / (1024 * 1024 * 1024),
			(uint64)PlatformMemoryStats.AvailableVirtual, double(PlatformMemoryStats.AvailableVirtual) / (1024 * 1024 * 1024),
			(uint64)PlatformMemoryStats.UsedPhysical, double(PlatformMemoryStats.UsedPhysical) / (1024 * 1024 * 1024),
			(uint64)PlatformMemoryStats.PeakUsedPhysical, double(PlatformMemoryStats.PeakUsedPhysical) / (1024 * 1024 * 1024),
			(uint64)PlatformMemoryStats.UsedVirtual, double(PlatformMemoryStats.UsedVirtual) / (1024 * 1024 * 1024),
			(uint64)PlatformMemoryStats.PeakUsedVirtual, double(PlatformMemoryStats.PeakUsedVirtual) / (1024 * 1024 * 1024)
		);
		if (BackupOOMMemoryPool && GLog)
		{
			UE::Private::GMalloc->DumpAllocatorStats(*GLog);
		}

		// let any registered handlers go
		FCoreDelegates::GetOutOfMemoryDelegate().Broadcast();

		// ErrorMsg might be unrelated to OoM error in some cases as the code that calls OnOutOfMemory could have called other system functions that modified errno
		UE_LOGF(LogMemory, Fatal, "Ran out of memory allocating %llu (%.1f MiB) bytes with alignment %u. Last error msg: %ls.", Size, double(Size) / (1024 * 1024), Alignment, ErrorMsg);
	};
	
	UE_CALL_ONCE(HandleOOM);
	FPlatformProcess::SleepInfinite(); // Unreachable
}

FMalloc* FGenericPlatformMemory::BaseAllocator()
{
	static FMalloc* Instance = nullptr;
	if (Instance != nullptr)
	{
		return Instance;
	}

	Instance = new FMallocAnsi();

	return Instance;
}

FPlatformMemoryStats FGenericPlatformMemory::GetStats()
{
	UE_LOGF(LogMemory, Warning, "FGenericPlatformMemory::GetStats not implemented on this platform");
	return FPlatformMemoryStats();
}

FPlatformMemoryStats FGenericPlatformMemory::GetStatsRaw()
{
	return FPlatformMemory::GetStats();
}

uint64 FGenericPlatformMemory::GetMemoryUsedFast()
{
	return FPlatformMemory::GetStats().UsedPhysical;
}

void FGenericPlatformMemory::GetStatsForMallocProfiler( FGenericMemoryStats& out_Stats )
{
#if	STATS
	FPlatformMemoryStats Stats = FPlatformMemory::GetStats();

	// Base common stats for all platforms.
	out_Stats.Add( GET_STATDESCRIPTION( STAT_TotalPhysical ), Stats.TotalPhysical );
	out_Stats.Add( GET_STATDESCRIPTION( STAT_TotalVirtual ), Stats.TotalVirtual );
	out_Stats.Add( GET_STATDESCRIPTION( STAT_PageSize ), Stats.PageSize );
	out_Stats.Add( GET_STATDESCRIPTION( STAT_TotalPhysicalGB ), (SIZE_T)Stats.TotalPhysicalGB );
	out_Stats.Add( GET_STATDESCRIPTION( STAT_AvailablePhysical ), Stats.AvailablePhysical );
	out_Stats.Add( GET_STATDESCRIPTION( STAT_AvailableVirtual ), Stats.AvailableVirtual );
	out_Stats.Add( GET_STATDESCRIPTION( STAT_UsedPhysical ), Stats.UsedPhysical );
	out_Stats.Add( GET_STATDESCRIPTION( STAT_PeakUsedPhysical ), Stats.PeakUsedPhysical );
	out_Stats.Add( GET_STATDESCRIPTION( STAT_UsedVirtual ), Stats.UsedVirtual );
	out_Stats.Add( GET_STATDESCRIPTION( STAT_PeakUsedVirtual ), Stats.PeakUsedVirtual );
#endif // STATS
}

const FPlatformMemoryConstants& FGenericPlatformMemory::GetConstants()
{
	UE_LOGF(LogMemory, Warning, "FGenericPlatformMemory::GetConstants not implemented on this platform");
	static FPlatformMemoryConstants MemoryConstants;
	return MemoryConstants;
}

uint32 FGenericPlatformMemory::GetPhysicalGBRam()
{
	return FPlatformMemory::GetConstants().TotalPhysicalGB;
}

bool FGenericPlatformMemory::PageProtect(void* const Ptr, const SIZE_T Size, const bool bCanRead, const bool bCanWrite)
{
	UE_LOGF(LogMemory, Verbose, "FGenericPlatformMemory::PageProtect not implemented on this platform");
	return false;
}


void FGenericPlatformMemory::DumpStats( class FOutputDevice& Ar )
{
	if (GenericPlatformMemory::GLogPlatformMemoryStats)
	{
	const float InvMB = 1.0f / 1024.0f / 1024.0f;
	FPlatformMemoryStats MemoryStats = FPlatformMemory::GetStats();
#if !NO_LOGGING
	const FName CategoryName(LogMemory.GetCategoryName());
#else
	const FName CategoryName("LogMemory");
#endif
	Ar.CategorizedLogf(CategoryName, ELogVerbosity::Log, TEXT("Platform Memory Stats for %s"), ANSI_TO_TCHAR(FPlatformProperties::PlatformName()));
	Ar.CategorizedLogf(CategoryName, ELogVerbosity::Log, TEXT("Process Physical Memory: %.2f MB used, %.2f MB peak"), (float)MemoryStats.UsedPhysical*InvMB, (float)MemoryStats.PeakUsedPhysical*InvMB);
	Ar.CategorizedLogf(CategoryName, ELogVerbosity::Log, TEXT("Process Virtual Memory: %.2f MB used, %.2f MB peak"), (float)MemoryStats.UsedVirtual*InvMB, (float)MemoryStats.PeakUsedVirtual*InvMB);

	Ar.CategorizedLogf(CategoryName, ELogVerbosity::Log, TEXT("Physical Memory: %.2f MB used,  %.2f MB free, %.2f MB total"), 
		(float)(MemoryStats.TotalPhysical - MemoryStats.AvailablePhysical)*InvMB, (float)MemoryStats.AvailablePhysical*InvMB, (float)MemoryStats.TotalPhysical*InvMB);
	Ar.CategorizedLogf(CategoryName, ELogVerbosity::Log, TEXT("Virtual Memory: %.2f MB used,  %.2f MB free, %.2f MB total"), 
		(float)(MemoryStats.TotalVirtual - MemoryStats.AvailableVirtual)*InvMB, (float)MemoryStats.AvailableVirtual*InvMB, (float)MemoryStats.TotalVirtual*InvMB);
	}
}

void FGenericPlatformMemory::DumpPlatformAndAllocatorStats( class FOutputDevice& Ar )
{
	FPlatformMemory::DumpStats( Ar );
	UE::Private::GMalloc->DumpAllocatorStats( Ar );
}

EPlatformMemorySizeBucket FGenericPlatformMemory::GetMemorySizeBucket()
{
	static bool bCalculatedBucket = false;
	static EPlatformMemorySizeBucket Bucket = EPlatformMemorySizeBucket::Default;

	// get bucket one time
	if (!bCalculatedBucket)
	{
		bCalculatedBucket = true;

		// get values for this platform from it's .ini
		int32 LargestMemoryGB=0, LargerMemoryGB=0, DefaultMemoryGB=0, SmallerMemoryGB=0,SmallestMemoryGB=0,TiniestMemoryGB=0;
		GConfig->GetInt(TEXT("PlatformMemoryBuckets"), TEXT("LargestMemoryBucket_MinGB"), LargestMemoryGB, GEngineIni);
		GConfig->GetInt(TEXT("PlatformMemoryBuckets"), TEXT("LargerMemoryBucket_MinGB"), LargerMemoryGB, GEngineIni);
		GConfig->GetInt(TEXT("PlatformMemoryBuckets"), TEXT("DefaultMemoryBucket_MinGB"), DefaultMemoryGB, GEngineIni);
		GConfig->GetInt(TEXT("PlatformMemoryBuckets"), TEXT("SmallerMemoryBucket_MinGB"), SmallerMemoryGB, GEngineIni);
		GConfig->GetInt(TEXT("PlatformMemoryBuckets"), TEXT("SmallestMemoryBucket_MinGB"), SmallestMemoryGB, GEngineIni);
        GConfig->GetInt(TEXT("PlatformMemoryBuckets"), TEXT("TiniestMemoryBucket_MinGB"), TiniestMemoryGB, GEngineIni);

		FPlatformMemoryStats Stats = FPlatformMemory::GetStats();

		// using the smaller of AddressLimit and TotalPhysical (some platforms may have AddressLimit be 
		// basically all virtual memory possible, some could use it to restrict to 32-bit process space)
		// @todo should we use a different stat?

#if PLATFORM_ANDROID
		// we don't exactly want to round up on android
		uint64 MemoryBucketRoundingAddition = 384;
		if (const FString* MemoryBucketRoundingAdditionVar = FPlatformMisc::GetConfigRulesVariable(TEXT("MemoryBucketRoundingAddition")))
		{
			MemoryBucketRoundingAddition = FCString::Atoi64(**MemoryBucketRoundingAdditionVar);
		}
		uint32 TotalPhysicalGB = (uint32)((Stats.TotalPhysical + MemoryBucketRoundingAddition * 1024 * 1024 - 1) / 1024 / 1024 / 1024);
#else
		uint32 TotalPhysicalGB = (uint32)((Stats.TotalPhysical + 1024 * 1024 * 1024 - 1) / 1024 / 1024 / 1024);
#endif
		uint32 AddressLimitGB = (uint32)((Stats.AddressLimit + 1024 * 1024 * 1024 - 1) / 1024 / 1024 / 1024);
		int32 CurMemoryGB = (int32)FMath::Min(TotalPhysicalGB, AddressLimitGB);

		// if at least Smaller is specified, we can set the Bucket
		if (SmallerMemoryGB > 0)
		{
			if (CurMemoryGB >= SmallerMemoryGB)
			{
				Bucket = EPlatformMemorySizeBucket::Smaller;
			}
			else if (CurMemoryGB >= SmallestMemoryGB)
			{
				Bucket = EPlatformMemorySizeBucket::Smallest;
			}
            else
            {
                Bucket = EPlatformMemorySizeBucket::Tiniest;
            }
		}
		if (DefaultMemoryGB > 0 && CurMemoryGB >= DefaultMemoryGB)
		{
			Bucket = EPlatformMemorySizeBucket::Default;
		}
		if (LargerMemoryGB > 0 && CurMemoryGB >= LargerMemoryGB)
		{
			Bucket = EPlatformMemorySizeBucket::Larger;
		}
		if (LargestMemoryGB > 0 && CurMemoryGB >= LargestMemoryGB)
		{
			Bucket = EPlatformMemorySizeBucket::Largest;
		}
		
		int32 BucketOverride = -1;
		if (FParse::Value(FCommandLine::Get(), TEXT("MemBucket="), BucketOverride))
		{
			Bucket = (EPlatformMemorySizeBucket)BucketOverride;
		}

        const TCHAR* BucketName = Bucket == EPlatformMemorySizeBucket::Tiniest ? TEXT("Tiniest") :
            Bucket == EPlatformMemorySizeBucket::Smallest ? TEXT("Smallest") :
			Bucket == EPlatformMemorySizeBucket::Smaller ? TEXT("Smaller") :
			Bucket == EPlatformMemorySizeBucket::Default ? TEXT("Default") :
			Bucket == EPlatformMemorySizeBucket::Larger ? TEXT("Larger") :
			TEXT("Largest");

		if (BucketOverride == -1)
		{
			UE_LOGF(LogHAL, Display, "Platform has ~ %d GB [%lld / %lld / %d], which maps to %ls [LargestMinGB=%d, LargerMinGB=%d, DefaultMinGB=%d, SmallerMinGB=%d, SmallestMinGB=0)",
				CurMemoryGB, Stats.TotalPhysical, Stats.AddressLimit, Stats.TotalPhysicalGB, BucketName, LargestMemoryGB, LargerMemoryGB, DefaultMemoryGB, SmallerMemoryGB);
		}
		else
		{
			UE_LOGF(LogHAL, Display, "Platform has ~ %d GB [%lld / %lld / %d], but commandline overrode bucket to %ls",
				CurMemoryGB, Stats.TotalPhysical, Stats.AddressLimit, Stats.TotalPhysicalGB, BucketName);
		}
	}

	return Bucket;
}





void FGenericPlatformMemory::MemswapGreaterThan8( void* RESTRICT Ptr1, void* RESTRICT Ptr2, SIZE_T Size )
{
	union PtrUnion
	{
		void*   PtrVoid;
		uint8*  Ptr8;
		uint16* Ptr16;
		uint32* Ptr32;
		uint64* Ptr64;
		UPTRINT PtrUint;
	};

	PtrUnion Union1 = { Ptr1 };
	PtrUnion Union2 = { Ptr2 };

	checkf(Union1.PtrVoid && Union2.PtrVoid, TEXT("Pointers must be non-null: %p, %p"), Union1.PtrVoid, Union2.PtrVoid);

	// We may skip up to 7 bytes below, so better make sure that we're swapping more than that
	// (8 is a common case that we also want to inline before we this call, so skip that too)
	check(Size > 8);

	if (Union1.PtrUint & 1)
	{
		Valswap(*Union1.Ptr8++, *Union2.Ptr8++);
		Size -= 1;
	}
	if (Union1.PtrUint & 2)
	{
		Valswap(*Union1.Ptr16++, *Union2.Ptr16++);
		Size -= 2;
	}
	if (Union1.PtrUint & 4)
	{
		Valswap(*Union1.Ptr32++, *Union2.Ptr32++);
		Size -= 4;
	}

	uint32 CommonAlignment = FMath::Min(FMath::CountTrailingZeros((uint32)(Union1.PtrUint - Union2.PtrUint)), 3u);
	switch (CommonAlignment)
	{
		default:
			for (; Size >= 8; Size -= 8)
			{
				Valswap(*Union1.Ptr64++, *Union2.Ptr64++);
			}

		case 2:
			for (; Size >= 4; Size -= 4)
			{
				Valswap(*Union1.Ptr32++, *Union2.Ptr32++);
			}

		case 1:
			for (; Size >= 2; Size -= 2)
			{
				Valswap(*Union1.Ptr16++, *Union2.Ptr16++);
			}

		case 0:
			for (; Size >= 1; Size -= 1)
			{
				Valswap(*Union1.Ptr8++, *Union2.Ptr8++);
			}
	}
}

FGenericPlatformMemory::FSharedMemoryRegion::FSharedMemoryRegion(const FString& InName, uint32 InAccessMode, void* InAddress, SIZE_T InSize)
	:	AccessMode(InAccessMode)
	,	Address(InAddress)
	,	Size(InSize)
{
	FCString::Strncpy(Name, *InName, UE_ARRAY_COUNT(Name));
}

FGenericPlatformMemory::FSharedMemoryRegion * FGenericPlatformMemory::MapNamedSharedMemoryRegion(const FString& Name, bool bCreate, uint32 AccessMode, SIZE_T Size)
{
	UE_LOGF(LogHAL, Error, "FGenericPlatformMemory::MapNamedSharedMemoryRegion not implemented on this platform");
	return NULL;
}

bool FGenericPlatformMemory::UnmapNamedSharedMemoryRegion(FSharedMemoryRegion * MemoryRegion)
{
	UE_LOGF(LogHAL, Error, "FGenericPlatformMemory::UnmapNamedSharedMemoryRegion not implemented on this platform");
	return false;
}


void FGenericPlatformMemory::InternalUpdateStats( const FPlatformMemoryStats& MemoryStats )
{
	// Generic method is empty. Implement at platform level.
}

bool FGenericPlatformMemory::IsExtraDevelopmentMemoryAvailable()
{
	return false;
}

uint64 FGenericPlatformMemory::GetExtraDevelopmentMemorySize()
{
	return 0;
}

bool FGenericPlatformMemory::GetLLMAllocFunctions(void*(*&OutAllocFunction)(size_t), void(*&OutFreeFunction)(void*, size_t), int32& OutAlignment)
{
	return false;
}

TArray<typename FGenericPlatformMemoryStats::FPlatformSpecificStat> FGenericPlatformMemoryStats::GetPlatformSpecificStats() const
{
	return TArray<FPlatformSpecificStat>();
}

uint64 FGenericPlatformMemoryStats::GetAvailablePhysical(bool bExcludeExtraDevMemory) const
{
	uint64 BytesAvailable = AvailablePhysical;

#if !UE_BUILD_SHIPPING
	if (bExcludeExtraDevMemory)
	{
		// FMath:Min to clamp at zero when ExtraDevelopmentMemory > AvailablePhysical
		BytesAvailable -= FMath::Min(FPlatformMemory::GetExtraDevelopmentMemorySize(), BytesAvailable);
	}
#endif

	return BytesAvailable;
}

FString FGenericPlatformMemory::PrettyMemory(uint64 Memory)
{
	static const TCHAR* Units[] = { TEXT("B"), TEXT("KB"), TEXT("MB"), TEXT("GB"), TEXT("TB"), TEXT("PB"), TEXT("EB") };
			
	uint64 UnitIndex = 0;
	uint64 Remainder = 0;

	while ((Memory > 1024) && (UnitIndex++ < (UE_ARRAY_COUNT(Units) - 1)))
	{
		Remainder = Memory & 1023;
		Memory >>= 10llu;
	}

	// Convert remainder to percentage
	const uint64 RemainderPerc = (Remainder * 100) >> 10;

	if (RemainderPerc)
	{
		if (RemainderPerc % 10)
		{
			return FString::Printf(TEXT("%llu.%02llu%s"), Memory, RemainderPerc, Units[UnitIndex]);
		}

		return FString::Printf(TEXT("%llu.%llu%s"), Memory, RemainderPerc / 10, Units[UnitIndex]);
	}

	return FString::Printf(TEXT("%llu%s"), Memory, Units[UnitIndex]);
}
