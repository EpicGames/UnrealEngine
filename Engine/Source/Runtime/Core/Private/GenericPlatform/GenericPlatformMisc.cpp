// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenericPlatform/GenericPlatformMisc.h"
#include "HAL/IConsoleManager.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CoreDelegates.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/CriticalSection.h"
#include "Misc/ScopeRWLock.h"
#include "Math/UnrealMathUtility.h"
#include "HAL/UnrealMemory.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Logging/LogMacros.h"
#include "CoreGlobals.h"
#include "Misc/Parse.h"
#include "Misc/CommandLine.h"
#include "Misc/Paths.h"
#include "Misc/PathViews.h"
#include "Misc/FileHelper.h"
#include "Internationalization/Text.h"
#include "Internationalization/Internationalization.h"
#include "Misc/Guid.h"
#include "Math/Color.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/App.h"
#include "GenericPlatform/GenericPlatformChunkInstall.h"
#include "GenericPlatform/GenericPlatformHostCommunication.h"
#include "HAL/FileManagerGeneric.h"
#include "Misc/VarargsHelper.h"
#include "Misc/SecureHash.h"
#include "HAL/ExceptionHandling.h"
#include "GenericPlatform/GenericPlatformCrashContext.h"
#include "GenericPlatform/GenericPlatformDriver.h"
#include "ProfilingDebugging/ExternalProfiler.h"
#include "HAL/LowLevelMemTracker.h"
#include "Templates/Function.h"
#include "Modules/ModuleManager.h"
#include "Misc/LazySingleton.h"
#include <atomic>
#include "Misc/UProjectInfo.h"
#include "Internationalization/Culture.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "Misc/CoreDelegates.h"
#include "AutoRTFM.h"
#include "FramePro/FrameProProfiler.h"

#if UE_ENABLE_ICU
	THIRD_PARTY_INCLUDES_START
		#include <unicode/locid.h>
	THIRD_PARTY_INCLUDES_END
#endif

#if !defined(PLATFORM_PROJECT_DIR_RELATIVE_TO_EXECUTABLE)
	#define PLATFORM_PROJECT_DIR_RELATIVE_TO_EXECUTABLE PLATFORM_DESKTOP
#endif

DEFINE_LOG_CATEGORY_STATIC(LogGenericPlatformMisc, Log, All);


#if (CSV_PROFILER_STATS && !UE_BUILD_SHIPPING)
bool GTrackCsvNamedEvents = false;
static FAutoConsoleVariableRef CVarTrackCsvNamedEvents(
	TEXT("r.TrackCsvNamedEvents"),
	GTrackCsvNamedEvents,
	TEXT("Whether to record named events in the csv profiler"),
	ECVF_Default
);
static std::atomic<uint32> GNamedEventMarkers(0L);
#endif


#if LOG_NAMED_EVENTS
struct FNameEventEntry
{
	FNameEventEntry()
		: FrameCount(0)
		, MaxFrameCount(0)
		, Total(0) {}

	uint64 FrameCount;
	uint64 MaxFrameCount;
	uint64 Total;
};

struct FLogNameEventStats
{
	enum class EOutputSort : uint8
	{
		SORTTYPE_Spike,
		SORTTYPE_Average,
		SORTTYPE_Total,
	};

	FLogNameEventStats()
	{
		MaxOutput = 100;
		SortingType = EOutputSort::SORTTYPE_Spike;
		StartFrame = 0;
	}

	void Init()
	{
		GConfig->GetArray(TEXT("SystemSettings"), TEXT("LogNamedEventFilters"), NamedEventExclusions, GEngineIni);
	}

	void Dump()
	{
		FScopeLock Lock(&NamedEventsCriticalSection);

		int32 LogCount = (MaxOutput < 0) ? NamedEventsMap.Num() : MaxOutput;
		float CurrentFrameCount = (float)GFrameCounter;
		
		UE_LOGF(LogGenericPlatformMisc, Log, "*********Log Named Events *********");
		UE_LOGF(LogGenericPlatformMisc, Log, "******* Top %i NamedEvents", LogCount);
		
		if (SortingType == EOutputSort::SORTTYPE_Total)
		{
			NamedEventsMap.ValueSort([&](const FNameEventEntry& A, const FNameEventEntry& B) { return A.Total > B.Total; });
		}
		else if (SortingType == EOutputSort::SORTTYPE_Average)
		{
			NamedEventsMap.ValueSort([&](const FNameEventEntry& A, const FNameEventEntry& B) { return (float)A.Total / (CurrentFrameCount - (float)StartFrame) > (float)B.Total / (CurrentFrameCount - (float)StartFrame); });
		}
		else 
		{
			NamedEventsMap.ValueSort([&](const FNameEventEntry& A, const FNameEventEntry& B) { return A.MaxFrameCount > B.MaxFrameCount; });
		}

		uint64 Total = 0;
		int32 Index = 0;

		for (const TPair<FName, FNameEventEntry>& Pair : NamedEventsMap)
		{
			if (Index <= LogCount)
			{
				UE_LOGF(LogGenericPlatformMisc, Log, "%ls : Max: %lu Total: %lu Avg: %f", *(Pair.Key.ToString()), Pair.Value.MaxFrameCount, Pair.Value.Total, (float)Pair.Value.Total / (CurrentFrameCount - (float)StartFrame));
			}
			Total += Pair.Value.Total;
			Index++;
		}
		UE_LOGF(LogGenericPlatformMisc, Log, "***********************************");
		UE_LOGF(LogGenericPlatformMisc, Log, "Total Unique NamedEvents: %i", NamedEventsMap.Num());
		UE_LOGF(LogGenericPlatformMisc, Log, "Average NamedEvents per frame: %f", (float)Total / (CurrentFrameCount - (float)StartFrame));
		UE_LOGF(LogGenericPlatformMisc, Log, "*********Log Named Events *********");
	}

	void Reset()
	{
		FScopeLock Lock(&NamedEventsCriticalSection);
		NamedEventsMap.Empty();
		StartFrame = GFrameCounter;
	}

	void ParseSortType(const FString& Arg)
	{
		if (Arg.Equals(TEXT("Avg"), ESearchCase::IgnoreCase))
		{
			SortingType = EOutputSort::SORTTYPE_Average;
		}
		else if (Arg.Equals(TEXT("Spike"), ESearchCase::IgnoreCase))
		{
			SortingType = EOutputSort::SORTTYPE_Spike;
		}
		else if(Arg.Equals(TEXT("Total"), ESearchCase::IgnoreCase))
		{
			SortingType = EOutputSort::SORTTYPE_Total;
		}
		else
		{
			UE_LOGF(LogGenericPlatformMisc, Log, "LogNamadEvent: Bad Sort argument. Option: Avg, Spike or Total");
		}
	}

	void LogEntry(const FString& Text)
	{
		// check for exclusion markers that we dont want to track each frames 
	    // Ex: Frame 1, Frame 2, Frame 3
		// You can specify +LogNamedEventFilters="Frame *" in the DefaultEngine.ini to exlude them 
		for (FString& Exclusion : NamedEventExclusions)
		{
			if (Text.MatchesWildcard(Exclusion))
			{
				return;
			}
		}
		FScopeLock Lock(&NamedEventsCriticalSection);
		FNameEventEntry& StatNameEvent = NamedEventsMap.FindOrAdd(FName(Text));
		StatNameEvent.FrameCount++;
	}

	void BeginFrame()
	{
		uint64 FrameCount = 0;
		FScopeLock Lock(&NamedEventsCriticalSection);
		for (TPair<FName, FNameEventEntry>& Pair : NamedEventsMap)
		{
			FNameEventEntry& Data = Pair.Value;
			if (Data.FrameCount > Data.MaxFrameCount)
			{
				Data.MaxFrameCount = Data.FrameCount;
			}
			Data.Total += Data.FrameCount;
			Data.FrameCount = 0;
		}
	}

	int32 MaxOutput;
	uint64 StartFrame;
	EOutputSort SortingType;

	TMap<FName, FNameEventEntry> NamedEventsMap;
	TArray<FString> NamedEventExclusions;
	FCriticalSection NamedEventsCriticalSection;
};
static FLogNameEventStats LogNameEventStats;

static FAutoConsoleCommand GLogNamedEventsCmd(
	TEXT("LogNamedEvents"),
	TEXT("Log named events Commands. LogNamedEvents Dump, Reset, Sort [Avg|Spike|Total], MaxOutput [int value]"),
		FConsoleCommandWithArgsDelegate::CreateStatic([](const TArray<FString>& Args)
		{
			bool bDumpNamedEvent = false;

			if (Args.Num() >= 1)
			{
				if (Args[0].Compare(FString("Reset"), ESearchCase::IgnoreCase) == 0)
				{
					LogNameEventStats.StartFrame = GFrameCounter;
					LogNameEventStats.Reset();
				}
				else if (Args[0].Compare(FString("Dump"), ESearchCase::IgnoreCase) == 0)
				{
					LogNameEventStats.Dump();
				}
				else if (Args[0].Compare(FString("Sort"), ESearchCase::IgnoreCase) == 0)
				{
					if (Args.Num() == 2)
					{
						LogNameEventStats.ParseSortType(Args[1]);
					}
				}
				else if (Args[0].Compare(FString("MaxOutput"), ESearchCase::IgnoreCase) == 0)
				{
					if (Args.Num() == 2)
					{
						LogNameEventStats.MaxOutput = FCString::Atoi(*Args[1]);
					}
				}
				else
				{
					UE_LOGF(LogGenericPlatformMisc, Log, "LogNamadEvent: Bad argument. Option: Reset, Dump, Sort [Avg,Sort,Total] or MaxOutput [int]");
				}
			}
		}
	)
);
#endif//LOG_NAMED_EVENTS

/** Holds an override path if a program has special needs */
static FString GOverrideProjectDir;

/** Hooks for moving ClipboardCopy and ClipboardPaste into FPlatformApplicationMisc */
CORE_API void (*ClipboardCopyShim)(const TCHAR* Text) = nullptr;
CORE_API void (*ClipboardPasteShim)(FString& Dest) = nullptr;

/* EBuildConfigurations interface
 *****************************************************************************/

bool LexTryParseString(EBuildConfiguration& OutConfiguration, const TCHAR* Configuration)
{
	if (FCString::Stricmp(Configuration, TEXT("Debug")) == 0)
	{
		OutConfiguration = EBuildConfiguration::Debug;
		return true;
	}
	else if (FCString::Stricmp(Configuration, TEXT("DebugGame")) == 0)
		{
		OutConfiguration = EBuildConfiguration::DebugGame;
		return true;
		}
	else if (FCString::Stricmp(Configuration, TEXT("Development")) == 0)
		{
		OutConfiguration = EBuildConfiguration::Development;
		return true;
		}
	else if (FCString::Stricmp(Configuration, TEXT("Shipping")) == 0)
		{
		OutConfiguration = EBuildConfiguration::Shipping;
		return true;
		}
	else if(FCString::Stricmp(Configuration, TEXT("Test")) == 0)
		{
		OutConfiguration = EBuildConfiguration::Test;
		return true;
		}
	else if(FCString::Stricmp(Configuration, TEXT("Unknown")) == 0)
		{
		OutConfiguration = EBuildConfiguration::Unknown;
		return true;
	}
	else
	{
		OutConfiguration = EBuildConfiguration::Unknown;
		return false;
		}
	}

const TCHAR* LexToString( EBuildConfiguration Configuration )
	{
		switch (Configuration)
		{
	case EBuildConfiguration::Debug:
				return TEXT("Debug");
	case EBuildConfiguration::DebugGame:
				return TEXT("DebugGame");
	case EBuildConfiguration::Development:
				return TEXT("Development");
	case EBuildConfiguration::Shipping:
				return TEXT("Shipping");
	case EBuildConfiguration::Test:
				return TEXT("Test");
			default:
				return TEXT("Unknown");
		}
	}

namespace EBuildConfigurations
{
	FText ToText( EBuildConfiguration Configuration )
	{
		switch (Configuration)
		{
		case EBuildConfiguration::Debug:
			return NSLOCTEXT("UnrealBuildConfigurations", "DebugName", "Debug");

		case EBuildConfiguration::DebugGame:
			return NSLOCTEXT("UnrealBuildConfigurations", "DebugGameName", "DebugGame");

		case EBuildConfiguration::Development:
			return NSLOCTEXT("UnrealBuildConfigurations", "DevelopmentName", "Development");

		case EBuildConfiguration::Shipping:
			return NSLOCTEXT("UnrealBuildConfigurations", "ShippingName", "Shipping");

		case EBuildConfiguration::Test:
			return NSLOCTEXT("UnrealBuildConfigurations", "TestName", "Test");

		default:
			return NSLOCTEXT("UnrealBuildConfigurations", "UnknownName", "Unknown");
		}
	}
}


/* EBuildTargetType functions
 *****************************************************************************/

bool LexTryParseString(EBuildTargetType& OutType, const TCHAR* Type)
{
	if (FCString::Strcmp(Type, TEXT("Editor")) == 0)
	{
		OutType = EBuildTargetType::Editor;
		return true;
	}
	else if (FCString::Strcmp(Type, TEXT("Game")) == 0)
	{
		OutType = EBuildTargetType::Game;
		return true;
	}
	else if (FCString::Strcmp(Type, TEXT("Server")) == 0)
{
		OutType = EBuildTargetType::Server;
		return true;
	}
	else if (FCString::Strcmp(Type, TEXT("Client")) == 0)
	{
		OutType = EBuildTargetType::Client;
		return true;
	}
	else if (FCString::Strcmp(Type, TEXT("Program")) == 0)
		{
		OutType = EBuildTargetType::Program;
		return true;
		}
	else if (FCString::Strcmp(Type, TEXT("Unknown")) == 0)
		{
		OutType = EBuildTargetType::Unknown;
		return true;
		}
	else
		{
		OutType = EBuildTargetType::Unknown;
		return false;
		}
	}

const TCHAR* LexToString(EBuildTargetType Type)
	{
	switch (Type)
		{
	case EBuildTargetType::Editor:
				return TEXT("Editor");
	case EBuildTargetType::Game:
				return TEXT("Game");
	case EBuildTargetType::Server:
				return TEXT("Server");
	case EBuildTargetType::Client:
		return TEXT("Client");
	case EBuildTargetType::Program:
		return TEXT("Program");
			default:
				return TEXT("Unknown");
		}
	}

EBuildTargetType EBuildTargets::FromString(const FString& Target)
{
	EBuildTargetType Type;
	LexTryParseString(Type, *Target);
	return Type;
}

const TCHAR* EBuildTargets::ToString(EBuildTargetType Target)
{
	return LexToString(Target);
}

FString FSHA256Signature::ToString() const
{
	FString LocalHashStr;
	for (int Idx = 0; Idx < 32; Idx++)
	{
		LocalHashStr += FString::Printf(TEXT("%02x"), Signature[Idx]);
	}
	return LocalHashStr;
}

const TCHAR* LexToString(ENetworkConnectionStatus EnumVal)
{
	switch (EnumVal)
	{
	case ENetworkConnectionStatus::Unknown:		return TEXT("Unknown");
	case ENetworkConnectionStatus::Disabled:	return TEXT("Disabled");
	case ENetworkConnectionStatus::Local:		return TEXT("Local");
	case ENetworkConnectionStatus::Connected:	return TEXT("Connected");
	}

	checkNoEntry();
	return TEXT("Unknown");
}

ENetworkConnectionStatus FGenericPlatformMisc::GetNetworkConnectionStatus()
{
	return CurrentNetworkConnectionStatus;
}

void FGenericPlatformMisc::SetNetworkConnectionStatus(ENetworkConnectionStatus NewNetworkConnectionStatus)
{
	const ENetworkConnectionStatus OldNetworkConnectionStatus = CurrentNetworkConnectionStatus;

	if (OldNetworkConnectionStatus != NewNetworkConnectionStatus)
	{
		CurrentNetworkConnectionStatus = NewNetworkConnectionStatus;

		QUICK_SCOPE_CYCLE_COUNTER(STAT_FGenericPlatformMisc_SetNetworkConnectionStatus);

		FCoreDelegates::OnNetworkConnectionStatusChanged.Broadcast(OldNetworkConnectionStatus, NewNetworkConnectionStatus);
	}
}

/* ENetworkConnectionType interface
 *****************************************************************************/

    const TCHAR* LexToString( ENetworkConnectionType Target )
    {
        switch (Target)
        {
            case ENetworkConnectionType::None:
                return TEXT("None");
                
            case ENetworkConnectionType::AirplaneMode:
                return TEXT("AirplaneMode");
                
            case ENetworkConnectionType::Cell:
                return TEXT("Cell");
                
            case ENetworkConnectionType::WiFi:
                return TEXT("WiFi");
                
            case ENetworkConnectionType::Ethernet:
                return TEXT("Ethernet");
                
			case ENetworkConnectionType::Bluetooth:
				return TEXT("Bluetooth");

			case ENetworkConnectionType::WiMAX:
				return TEXT("WiMAX");

			default:
                return TEXT("Unknown");
        }
    }

/* FGenericPlatformMisc interface
 *****************************************************************************/

#if !UE_BUILD_SHIPPING
	bool FGenericPlatformMisc::bShouldPromptForRemoteDebugging = false;
	bool FGenericPlatformMisc::bPromptForRemoteDebugOnEnsure = false;
#endif	//#if !UE_BUILD_SHIPPING

EDeviceScreenOrientation FGenericPlatformMisc::AllowedDeviceOrientation = EDeviceScreenOrientation::Unknown;

ENetworkConnectionStatus FGenericPlatformMisc::CurrentNetworkConnectionStatus = ENetworkConnectionStatus::Connected;

struct FGenericPlatformMisc::FStaticData
{
	FString         RootDir;
	TArray<FString> AdditionalRootDirectories;
	FRWLock         AdditionalRootDirectoriesLock;
	FString         EngineDirectory;
	FString         LaunchDir;
	FString         ProjectDir;
	FString         GamePersistentDownloadDir;
};

FString FGenericPlatformMisc::GetEnvironmentVariable(const TCHAR* VariableName)
{
	return FString();
}

void FGenericPlatformMisc::SetEnvironmentVar(const TCHAR* VariableName, const TCHAR* Value)
{
	UE_LOGF(LogGenericPlatformMisc, Error, "SetEnvironmentVar not implemented for this platform: %ls = %ls", VariableName, Value);
}

const TCHAR* FGenericPlatformMisc::GetPathVarDelimiter()
{
	return TEXT(";");
}

TArray<uint8> FGenericPlatformMisc::GetMacAddress()
{
	return TArray<uint8>();
}

FString FGenericPlatformMisc::GetMacAddressString()
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	TArray<uint8> MacAddr = FPlatformMisc::GetMacAddress();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	FString Result;
	for (TArray<uint8>::TConstIterator it(MacAddr);it;++it)
	{
		Result += FString::Printf(TEXT("%02x"),*it);
	}
	return Result;
}

FString FGenericPlatformMisc::GetHashedMacAddressString()
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// ensure empty MAC addresses don't return a hash of zero bytes.
	FString MacAddr = FPlatformMisc::GetMacAddressString();
	if (!MacAddr.IsEmpty())
	{
		return FMD5::HashAnsiString(*MacAddr);
	}
	else
	{
		return FString();
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

FString FGenericPlatformMisc::GetDeviceId()
{
	// not implemented at the base level. Each platform must decide how to implement this, if possible.
	return FString();
}

FString FGenericPlatformMisc::GetUniqueAdvertisingId()
{
	// this has no meaning generically, primarily used for attribution on mobile platforms
	return FString();
}

void FGenericPlatformMisc::SubmitErrorReport( const TCHAR* InErrorHist, EErrorReportMode::Type InMode )
{
	if ((!FPlatformMisc::IsDebuggerPresent() || GAlwaysReportCrash) && !FParse::Param(FCommandLine::Get(), TEXT("CrashForUAT")))
	{
		if ( GUseCrashReportClient )
		{
			int32 FromCommandLine = 0;
			FParse::Value( FCommandLine::Get(), TEXT("AutomatedPerfTesting="), FromCommandLine );
			if (FApp::IsUnattended() && FromCommandLine != 0 && FParse::Param(FCommandLine::Get(), TEXT("KillAllPopUpBlockingWindows")))
			{
				UE_LOGF(LogGenericPlatformMisc, Error, "This platform does not implement KillAllPopUpBlockingWindows");
			}
		}
		else
		{
			UE_LOGF(LogGenericPlatformMisc, Error, "This platform cannot submit a crash report. Report was:\n%ls", InErrorHist);
		}
	}
}

EProcessDiagnosticFlags FGenericPlatformMisc::GetProcessDiagnostics()
{
	static EProcessDiagnosticFlags FoundDiagnostics = []() -> EProcessDiagnosticFlags
	{
		EProcessDiagnosticFlags Result = EProcessDiagnosticFlags::None;
		const TCHAR* CommandLine = FCommandLine::Get();

		if (FCString::Stristr(CommandLine, TEXT("-ansimalloc")))
		{
			Result |= EProcessDiagnosticFlags::AnsiMalloc;
		}

		if (FCString::Stristr(CommandLine, TEXT("-stompmalloc")))
		{
			Result |= EProcessDiagnosticFlags::StompMalloc;
		}

		return Result;
	}();

	return FoundDiagnostics;
}


#if PLATFORM_CPU_X86_FAMILY

#ifdef _MSC_VER
#include <intrin.h>
#define CpuIdEx __cpuidex
#define CpuId __cpuid
static inline uint64 xgetbv(uint32 xcr)
{
#if _MSC_VER >= 1500 // VC++2008 or later
	return _xgetbv(xcr);
#else
	return 0;
#endif
}
#else
// GCC/Clang

// 64-bit: GCC/Clang won't let us use "=b" constraint on Mac64, and we need to preserve RBX
// (PIC/PIE base)
#define CpuIdEx(out, leaf_id, subleaf_id)\
			asm("xchgq %%rbx,%q1\n" \
				"cpuid\n" \
				"xchgq %%rbx,%q1\n" \
				: "=a" (out[0]), "=&r" (out[1]), "=c" (out[2]), "=d" (out[3]): "0" (leaf_id), "2"(subleaf_id));

#define CpuId(out, leaf_id) CpuIdEx(out, leaf_id, 0)
static inline uint64 xgetbv(uint32 xcr)
{
	uint32 lo, hi;
	__asm__ __volatile__("xgetbv" : "=a"(lo), "=d"(hi) : "c"(xcr));
	return ((uint64)hi << 32) | lo;
}
#endif // if not msc

static std::atomic_uint32_t CachedX86FeatureBits = 0;
uint32 FGenericPlatformMisc::GetFeatureBits_X86()
{
	//
	// Note we are 64bit+ now so we know we have cpuid.
	//

	uint32 FeatureBits = CachedX86FeatureBits.load(std::memory_order_relaxed);
	if (FeatureBits)
	{
		return FeatureBits;
	}

	int CpuInfo[4];
	uint32 MaxLeaf;

	// Basic CPUID information
	CpuId(CpuInfo, 0);
	MaxLeaf = CpuInfo[0];

	// Basic feature flags
	CpuId(CpuInfo, 1);

	FeatureBits |= (CpuInfo[3] & (1u << 26)) ? ECPUFeatureBits_X86::SSE2 : 0;
	FeatureBits |= (CpuInfo[2] & (1u << 9)) ? ECPUFeatureBits_X86::SSSE3 : 0;
	FeatureBits |= (CpuInfo[2] & (1u << 20)) ? ECPUFeatureBits_X86::SSE42 : 0;

	// Feature flags we don't report directly (because they need further checks), but use internally
	bool bHasFMA = (CpuInfo[2] & (1u << 12)) != 0;
	bool bHasPopcnt = (CpuInfo[2] & (1u << 23)) != 0;
	bool bHasOsXsave = (CpuInfo[2] & (1u << 27)) != 0;
	bool bHasAVX = (CpuInfo[2] & (1u << 28)) != 0;
	bool bHasF16C = (CpuInfo[2] & (1u << 29)) != 0;

	// Get XCR0 if available (context save bits) - required for AVX and other extensions
	// to test this, on Admin command prompt: (WARNING: this can brick your machine
	// depending on your drivers. You might need to boot from a recovery/install USB
	// image to undo if that's the case!)
	//     bcdedit /set xsavedisable 1
	// (be sure to "/set xsavedisable 0" again once you're done testing)
	uint64 Xcr0 = 0;
	if (bHasOsXsave)
	{
		Xcr0 = xgetbv(0);
	}

	// Required context save bits to use AVX and AVX-512, respectively
	bool bAVXContextSupport = (Xcr0 & 0x6) == 0x6;
	bool bAVX512ContextSupport = (Xcr0 & 0xe6) == 0xe6;

	// Combine these to determine effective AVX support
	if (bAVXContextSupport)
	{
		if (bHasAVX)
		{
			FeatureBits |= ECPUFeatureBits_X86::AVX;
		}

		// Even though F16C does not require using AVX registers, the relevant instructions
		// fault if XSAVE support is disabled
		if (bHasF16C)
		{
			FeatureBits |= ECPUFeatureBits_X86::F16C;
		}

		if (bHasFMA)
		{
			FeatureBits |= ECPUFeatureBits_X86::FMA;
		}
	}

	if (MaxLeaf >= 7)
	{
		// "Structured extended feature flags enumeration"
		CpuIdEx(CpuInfo, 7, 0);

		// Some (Celeron) Skylakes erroneously report BMI1/BMI2 even though they don't have it.
		// These Celerons also don't have AVX.
		//
		// All CPUs that actually have BMI1/BMI2 (as of this writing, 2016-05-11) have AVX.
		// (The ones we care about, anyway.) So only report BMI1/BMI2 if AVX is present.
		// Also only report AVX or the BMIs if POPCNT is present; all processors I know of
		// have either both or neither, and it's convenient for us to be able to assume
		// that either BMI1/BMI2 or AVX2 implies POPCNT.
		if (bAVXContextSupport && bHasAVX && bHasPopcnt)
		{
			// These don't actually require XSAVE support to use, but let's not increase
			// the size of the testing matrix further.
			FeatureBits |= (CpuInfo[1] & (1u << 3)) ? ECPUFeatureBits_X86::BMI1 : 0;
			FeatureBits |= (CpuInfo[1] & (1u << 8)) ? ECPUFeatureBits_X86::BMI2 : 0;

			// In addition to the above, only report AVX2 if BMI1 (and thus LZCNT/TZCNT)
			// are also reported present; finally VC++ with /arch:AVX2 will emit BMI2
			// instructions for things like variable shifts so we require BMI2 for AVX2
			// as well.
			//
			// In practice this is not a limitation, AVX2 and BMI2 are a package deal on
			// all uArchs I'm aware of.
			const uint32 Avx2Bits = (1u << 3) /* BMI1 */ | (1u << 5) /* AVX2 */ | (1u << 8) /* BMI2 */;
			if ((CpuInfo[1] & Avx2Bits) == Avx2Bits)
			{
				FeatureBits |= ECPUFeatureBits_X86::AVX2;
			}

			if (bAVX512ContextSupport)
			{
				// For us to report AVX512, we want the Skylake feature set
				const uint32 Avx512Bits = (1u << 31) /* AVX512VL */ | (1u << 30) /* AVX512BW */ | (1u << 17) /* AVX512DQ */ | (1u << 16) /* AVX512F */;
				if ((CpuInfo[1] & Avx512Bits) == Avx512Bits)
				{
					FeatureBits |= ECPUFeatureBits_X86::AVX512;
				}

				// Use the VBMI2 bit (set on ICL+) to set the NOCAVEATS flag. This is available
				// on a generation of cores where AVX-512 has no major clock penalty anymore so
				// whether to use AVX-512 or not is a much more straightforward calculation,
				// and not so dependent on what else is running at the same time.
				if (CpuInfo[2] & (1u << 6))
				{
					FeatureBits |= ECPUFeatureBits_X86::AVX512_NOCAVEATS;
				}
			}
		}
	}

	// write detected features
	// only write value once at end of the function!
	FeatureBits |= 1; // initialized flag

	CachedX86FeatureBits.store(FeatureBits, std::memory_order_release);
	return FeatureBits;
}
#undef CpuIdEx
#undef CpuId
#else

uint32 FGenericPlatformMisc::GetFeatureBits_X86()
{
	return 0;
}

#endif //PLATFORM_CPU_X86_FAMILY

bool FGenericPlatformMisc::HasAVX2InstructionSupport()
{
	return CheckFeatureBit_X86(ECPUFeatureBits_X86::AVX2);
}

FString FGenericPlatformMisc::GetCPUVendor()
{
	// Not implemented cross-platform. Each platform may or may not choose to implement this.
	return FString( "GenericCPUVendor" );
}

FString FGenericPlatformMisc::GetCPUBrand()
{
	// Not implemented cross-platform. Each platform may or may not choose to implement this.
	return FString( "GenericCPUBrand" );
}

uint32 FGenericPlatformMisc::GetCPUModelID()
{
	// Not implemented cross-platform. Each platform may or may not choose to implement this.
	return 0;
}

uint32 FGenericPlatformMisc::GetCPUMicrocodeRevision()
{
	// Not implemented cross-platform. Each platform may or may not choose to implement this.
	return 0;
}

FString FGenericPlatformMisc::GetCPUChipset()
{
	return FString( "Unknown" );
}

uint32 FGenericPlatformMisc::GetCPUInfo()
{
	// Not implemented cross-platform. Each platform may or may not choose to implement this.
	return 0;
}

FString FGenericPlatformMisc::GetHostArchitecture()
{
	return FString();
}

FString FGenericPlatformMisc::GetPrimaryGPUBrand()
{
	// Not implemented cross-platform. Each platform may or may not choose to implement this.
	return FString( "GenericGPUBrand" );
}

FString FGenericPlatformMisc::GetDeviceMakeAndModel()
{
	const FString CPUVendor = FPlatformMisc::GetCPUVendor().TrimStartAndEnd();
	const FString CPUBrand = FPlatformMisc::GetCPUBrand().TrimStartAndEnd();
	const FString CPUChipset = FPlatformMisc::GetCPUChipset().TrimStartAndEnd();
	if (!CPUChipset.Equals(TEXT("Unknown")))
	{
		if (CPUBrand.Contains(TEXT("|")))
		{
			FString FixedCPUBrand = CPUBrand.Replace(TEXT("|"), TEXT("_"));
			return FString::Printf(TEXT("%s|%s|%s"), *CPUVendor, *FixedCPUBrand, *CPUChipset);
		}
		return FString::Printf(TEXT("%s|%s|%s"), *CPUVendor, *CPUBrand, *CPUChipset);
	}
	return FString::Printf(TEXT("%s|%s"), *CPUVendor, *CPUBrand);
}

FGPUDriverInfo FGenericPlatformMisc::GetGPUDriverInfo(const FString& DeviceDescription)
{
	return FGPUDriverInfo();
}

void FGenericPlatformMisc::GetOSVersions( FString& out_OSVersionLabel, FString& out_OSSubVersionLabel )
{
	// Not implemented cross-platform. Each platform may or may not choose to implement this.
	out_OSVersionLabel = FString( "GenericOSVersionLabel" );
	out_OSSubVersionLabel = FString( "GenericOSSubVersionLabel" );
}


FString FGenericPlatformMisc::GetOSVersion()
{
	return FString();
}

bool FGenericPlatformMisc::GetDiskTotalAndFreeSpace( const FString& InPath, uint64& TotalNumberOfBytes, uint64& NumberOfFreeBytes )
{
	// Not implemented cross-platform. Each platform may or may not choose to implement this.
	TotalNumberOfBytes = 0;
	NumberOfFreeBytes = 0;
	return false;
}


void FGenericPlatformMisc::MemoryBarrier()
{
}

void FGenericPlatformMisc::RaiseException(uint32 ExceptionCode)
{
	/** This is the last place to gather memory stats before exception. */
	FGenericCrashContext::SetMemoryStats(FPlatformMemory::GetStats());

	*((uint32*)3) = ExceptionCode;
}

template<typename CharType>
void FGenericPlatformMisc::StatNamedEvent(const CharType* Text)
{
#if (CSV_PROFILER_STATS && !UE_BUILD_SHIPPING)
	if (GTrackCsvNamedEvents)
	{
		++GNamedEventMarkers;
	}
#endif
#if LOG_NAMED_EVENTS
	FString NamedEvent(Text);
	LogNameEventStats.LogEntry(NamedEvent);
#endif
}

template CORE_API void FGenericPlatformMisc::StatNamedEvent<ANSICHAR>(const ANSICHAR* Text);
template CORE_API void FGenericPlatformMisc::StatNamedEvent<TCHAR>(const TCHAR* Text);

void FGenericPlatformMisc::TickStatNamedEvents()
{
#if (CSV_PROFILER_STATS && !UE_BUILD_SHIPPING)
	if (GTrackCsvNamedEvents)
	{
		int32 NamedEventCount = GNamedEventMarkers.exchange(0);
		CSV_CUSTOM_STAT_GLOBAL(NamedEventMarkers, NamedEventCount, ECsvCustomStatOp::Set);
	}
#endif
#if LOG_NAMED_EVENTS
	LogNameEventStats.BeginFrame();
#endif 
}

void FGenericPlatformMisc::LogNameEventStatsInit()
{
#if LOG_NAMED_EVENTS
	LogNameEventStats.Init();
#endif
}

#if ENABLE_GENERIC_NAMED_EVENTS

void FGenericPlatformMisc::BeginNamedEventFrame()
{
	FGenericPlatformMisc::TickStatNamedEvents();
#if FRAMEPRO_ENABLED
	FFrameProProfiler::FrameStart();
#endif
}

void FGenericPlatformMisc::BeginNamedEvent(const struct FColor& Color, const ANSICHAR* Text)
{
#if UE_EXTERNAL_PROFILING_ENABLED
	FExternalProfilerTrace::StartScopedEvent(Color, Text);
#endif
	StatNamedEvent(Text);
#if FRAMEPRO_ENABLED
	FFrameProProfiler::PushEvent(Text);
#endif
}

void FGenericPlatformMisc::BeginNamedEvent(const struct FColor& Color, const TCHAR* Text)
{
#if UE_EXTERNAL_PROFILING_ENABLED
	FExternalProfilerTrace::StartScopedEvent(Color, Text);
#endif
	StatNamedEvent(Text);
#if FRAMEPRO_ENABLED
	FFrameProProfiler::PushEvent(Text);
#endif
}

void FGenericPlatformMisc::EndNamedEvent()
{
#if UE_EXTERNAL_PROFILING_ENABLED
	FExternalProfilerTrace::EndScopedEvent();
#endif
#if FRAMEPRO_ENABLED
	FFrameProProfiler::PopEvent();
#endif
}

void FGenericPlatformMisc::CustomNamedStat(const TCHAR* Text, float Value, const TCHAR* Graph, const TCHAR* Unit)
{
	FRAMEPRO_DYNAMIC_CUSTOM_STAT(TCHAR_TO_WCHAR(Text), Value, TCHAR_TO_WCHAR(Graph), TCHAR_TO_WCHAR(Unit), FRAMEPRO_COLOUR(255,255,255) );
}

void FGenericPlatformMisc::CustomNamedStat(const ANSICHAR* Text, float Value, const ANSICHAR* Graph, const ANSICHAR* Unit)
{
	FRAMEPRO_DYNAMIC_CUSTOM_STAT(Text, Value, Graph, Unit, FRAMEPRO_COLOUR(255,255,255));
}

#endif // ENABLE_GENERIC_NAMED_EVENTS

bool FGenericPlatformMisc::SetStoredValues(const FString& InStoreId, const FString& InSectionName, const TMap<FString, FString>& InKeyValues)
{
	for (const TPair<FString, FString>& InKeyValue : InKeyValues)
	{
		if (!FPlatformMisc::SetStoredValue(InStoreId, InSectionName, InKeyValue.Key, InKeyValue.Value))
		{
			return false;
		}
	}

	return true;
}

bool FGenericPlatformMisc::SetStoredValue(const FString& InStoreId, const FString& InSectionName, const FString& InKeyName, const FString& InValue)
{
	check(!InStoreId.IsEmpty());
	check(!InSectionName.IsEmpty());
	check(!InKeyName.IsEmpty());

	// This assumes that FPlatformProcess::ApplicationSettingsDir() returns a user-specific directory; it doesn't on Windows, but Windows overrides this behavior to use the registry
	const FString ConfigPath = FString(FPlatformProcess::ApplicationSettingsDir()) / InStoreId / FString("KeyValueStore.ini");
		
	FConfigFile ConfigFile;
	ConfigFile.Read(ConfigPath);
	// update one entry
	ConfigFile.SetString(*InSectionName, *InKeyName, *InValue);
	return ConfigFile.Write(ConfigPath);
}

bool FGenericPlatformMisc::GetStoredValue(const FString& InStoreId, const FString& InSectionName, const FString& InKeyName, FString& OutValue)
{
	check(!InStoreId.IsEmpty());
	check(!InSectionName.IsEmpty());
	check(!InKeyName.IsEmpty());

	// This assumes that FPlatformProcess::ApplicationSettingsDir() returns a user-specific directory; it doesn't on Windows, but Windows overrides this behavior to use the registry
	const FString ConfigPath = FString(FPlatformProcess::ApplicationSettingsDir()) / InStoreId / FString("KeyValueStore.ini");
		
	FConfigFile ConfigFile;
	ConfigFile.Read(ConfigPath);

	return ConfigFile.GetString(*InSectionName, *InKeyName, OutValue);
}

bool FGenericPlatformMisc::DeleteStoredValue(const FString& InStoreId, const FString& InSectionName, const FString& InKeyName)
{
	check(!InStoreId.IsEmpty());
	check(!InSectionName.IsEmpty());
	check(!InKeyName.IsEmpty());

	// This assumes that FPlatformProcess::ApplicationSettingsDir() returns a user-specific directory; it doesn't on Windows, but Windows overrides this behavior to use the registry
	const FString ConfigPath = FString(FPlatformProcess::ApplicationSettingsDir()) / InStoreId / FString("KeyValueStore.ini");

	FConfigFile ConfigFile;
	ConfigFile.Read(ConfigPath);

	if (ConfigFile.RemoveKeyFromSection(*InSectionName, *InKeyName))
	{
		return ConfigFile.Write(ConfigPath);
	}

	return false;
}

bool FGenericPlatformMisc::DeleteStoredSection(const FString& InStoreId, const FString& InSectionName)
{
	check(!InStoreId.IsEmpty());
	check(!InSectionName.IsEmpty());

	// This assumes that FPlatformProcess::ApplicationSettingsDir() returns a user-specific directory; it doesn't on Windows, but Windows overrides this behavior to use the registry
	const FString ConfigPath = FString(FPlatformProcess::ApplicationSettingsDir()) / InStoreId / FString("KeyValueStore.ini");

	FConfigFile ConfigFile;
	ConfigFile.Read(ConfigPath);

	if (ConfigFile.Remove(InSectionName) != 0)
	{
		ConfigFile.Dirty = true;
		return ConfigFile.Write(ConfigPath);
	}

	return false;
}

void FGenericPlatformMisc::LowLevelOutputDebugString( const TCHAR *Message )
{
	FPlatformMisc::LocalPrint( Message );
}

void FGenericPlatformMisc::LowLevelOutputDebugStringf(const TCHAR *Fmt, ... )
{
	GROWABLE_LOGF(
		FPlatformMisc::LowLevelOutputDebugString( Buffer );
	);
}

bool FGenericPlatformMisc::IsLowLevelOutputDebugStringStructured()
{
	if (!FCommandLine::IsInitialized())
	{
		return false;
	}
	static bool bJsonDebugOutput = FParse::Param(FCommandLine::Get(), TEXT("JsonDebugOutput"));
	return bJsonDebugOutput;
}

void FGenericPlatformMisc::SetUTF8Output()
{
	// assume that UTF-8 is possible by default anyway
}

void FGenericPlatformMisc::LocalPrint( const TCHAR* Str )
{
#if PLATFORM_TCHAR_IS_CHAR16
	printf("%s", TCHAR_TO_UTF8(Str));
#elif PLATFORM_USE_LS_SPEC_FOR_WIDECHAR
	printf("%ls", Str);
#else
	wprintf(TEXT("%s"), Str);
#endif
}

bool FGenericPlatformMisc::HasSeparateChannelForDebugOutput()
{
	return true;
}

void FGenericPlatformMisc::RequestExit( bool Force, const TCHAR* CallSite)
{
	UE_LOGF(LogGenericPlatformMisc, Log,  "FPlatformMisc::RequestExit(%i, %ls)", Force,
		CallSite ? CallSite : TEXT("<NoCallSiteInfo>"));
	if( Force )
	{
		// Force immediate exit.
		// Dangerous because config code isn't flushed, global destructors aren't called, etc.
		// Suppress abort message and MS reports.
		abort();
	}
	else
	{
		// Tell the platform specific code we want to exit cleanly from the main loop.
		RequestEngineExit(TEXT("GenericPlatform RequestExit"));
	}
}

bool FGenericPlatformMisc::RestartApplicationWithCmdLine(const char* CmdLine)
{
	UE_LOGF(LogInit, Display, "Restart application (with cmdnline) is not supported or implemented in current platform");
	return false;
}

bool FGenericPlatformMisc::RestartApplication()
{
	UE_LOGF(LogInit, Display, "Restart application is not supported or implemented in current platform");
	return false;
}

void FGenericPlatformMisc::RequestExitWithStatus(bool Force, uint8 ReturnCode, const TCHAR* CallSite)
{
	// Generic implementation will ignore the return code - this may be important, so warn.
	UE_LOGF(LogGenericPlatformMisc, Warning, "FPlatformMisc::RequestExitWithStatus(%i, %d) - return code will be ignored by the generic implementation.", Force, ReturnCode);

	return FPlatformMisc::RequestExit(Force, CallSite);
}

const TCHAR* FGenericPlatformMisc::GetSystemErrorMessage(TCHAR* OutBuffer, int32 BufferCount, int32 Error)
{
	const TCHAR* Message = TEXT("No system errors available on this platform.");
	check(OutBuffer && BufferCount > 80);
	Error = 0;
	FCString::Strncpy(OutBuffer, Message, BufferCount);
	return OutBuffer;
}

void FGenericPlatformMisc::CreateGuid(FGuid& Guid)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FGenericPlatformMisc_CreateGuid);
	
	// this is a pretty terrible Guid maker that has way less than 128 bits of randomness
	// luckily this is not used, as every platform has an override
	//   do not use this!
	// all the timing used for "randomness" can easily be the same on two threads
	// you wind up with 15 bits of randomness from the stdlib rand() call
	//   the atomic IncrementCounter is crucial to ensure simultaneous calls to CreateGuid
	//	 on different threads don't produce the same Guid here

	// note the first calls to this are in static initializers, people use it to init globals

	static std::atomic<uint32> IncrementCounter = 0; 
	static FDateTime InitialDateTime;
	static uint64 InitialCycleCounter;

	FDateTime EstimatedCurrentDateTime;

	uint32 SequentialBits = IncrementCounter.fetch_add(1);

	if (SequentialBits == 0) // not a thread-safe init, but our first call is not threaded, so okay
	{
		// Hack: First Guid can be created prior to FPlatformTime::InitTiming(), so do it here.
		//	InitTiming is done in static init in CoreGlobals.cpp
		//	but CreateGuid is also used in static initializers, so we may come first
		//  this can result in InitTiming being called twice which is not entirely benign
		FPlatformTime::InitTiming();

		// uses FPlatformTime::SystemTime()
		InitialDateTime = FDateTime::Now();
		InitialCycleCounter = FPlatformTime::Cycles64();
		
		EstimatedCurrentDateTime = InitialDateTime;
	}
	else
	{
		FTimespan ElapsedTime = FTimespan::FromSeconds(FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - InitialCycleCounter));
		
		EstimatedCurrentDateTime = InitialDateTime + ElapsedTime;
	}

	uint32 RandBits = FMath::Rand();
	
	// bit rotate 16 : 
	SequentialBits = ( SequentialBits << 16 ) | ( SequentialBits >> 16 );

	Guid = FGuid(RandBits ^ SequentialBits, EstimatedCurrentDateTime.GetTicks() >> 32, EstimatedCurrentDateTime.GetTicks() & 0xffffffff, FPlatformTime::Cycles());

	// note: all the platform Guid makers do this but we do not :
	//	Result[1] = (Result[1] & 0xffff0fff) | 0x00004000; // version 4
	//	Result[2] = (Result[2] & 0x3fffffff) | 0x80000000; // variant 1
}

const TCHAR* LexToString( EAppReturnType::Type Value )
{
	switch (Value)
	{
	case EAppReturnType::No:
		return TEXT("No");
	case EAppReturnType::Yes:
		return TEXT("Yes");
	case EAppReturnType::YesAll:
		return TEXT("YesAll");
	case EAppReturnType::NoAll:
		return TEXT("NoAll");
	case EAppReturnType::Cancel:
		return TEXT("Cancel");
	case EAppReturnType::Ok:
		return TEXT("Ok");
	case EAppReturnType::Retry:
		return TEXT("Retry");
	case EAppReturnType::Continue:
		return TEXT("Continue");
	default:
		return TEXT("Unknown");
	}
}

EAppReturnType::Type FGenericPlatformMisc::MessageBoxExt( EAppMsgType::Type MsgType, const TCHAR* Text, const TCHAR* Caption )
{
	// a message box typically conveys important information, so try to make sure at least one message reaches the user
	UE_LOGF(LogGenericPlatformMisc, Warning, "MessageBox: %ls : %ls", Caption, Text);
	FPlatformMisc::LowLevelOutputDebugStringf(TEXT("MessageBox: %s : %s\n"), Caption, Text);

	switch(MsgType)
	{
	case EAppMsgType::Ok:
		return EAppReturnType::Ok; // Ok
	case EAppMsgType::YesNo:
		return EAppReturnType::No; // No
	case EAppMsgType::OkCancel:
		return EAppReturnType::Cancel; // Cancel
	case EAppMsgType::YesNoCancel:
		return EAppReturnType::Cancel; // Cancel
	case EAppMsgType::CancelRetryContinue:
		return EAppReturnType::Cancel; // Cancel
	case EAppMsgType::YesNoYesAllNoAll:
		return EAppReturnType::No; // No
	case EAppMsgType::YesNoYesAllNoAllCancel:
		return EAppReturnType::Yes; // Yes
	default:
		check(0);
	}
	return EAppReturnType::Cancel; // Cancel
}

const TCHAR* FGenericPlatformMisc::RootDir()
{
	FString& Path = TLazySingleton<FStaticData>::Get().RootDir;
	if (Path.Len() == 0)
	{
		FString TempPath = FPaths::EngineDir();
		int32 chopPos = TempPath.Find(TEXT("/Engine"), ESearchCase::IgnoreCase, ESearchDir::FromEnd);
		if (chopPos != INDEX_NONE)
		{
			TempPath.LeftInline(chopPos + 1, EAllowShrinking::No);
		}
		else
		{
			TempPath = FPlatformProcess::BaseDir();

			// if the path ends in a separator, remove it
			if (TempPath.Right(1) == TEXT("/"))
			{
				TempPath.LeftChopInline(1, EAllowShrinking::No);
			}

			// keep going until we've removed Binaries
#if IS_MONOLITHIC && !IS_PROGRAM
			int32 pos = TempPath.Find(*FString::Printf(TEXT("/%s/Binaries"), FApp::GetProjectName()));
#else
			int32 pos = TempPath.Find(TEXT("/Engine/Binaries"), ESearchCase::IgnoreCase, ESearchDir::FromEnd);
#endif
			if (pos != INDEX_NONE)
			{
				TempPath.LeftInline(pos + 1, EAllowShrinking::No);
			}
			else
			{
				pos = TempPath.Find(TEXT("/../Binaries"), ESearchCase::IgnoreCase, ESearchDir::FromEnd);
				if (pos != INDEX_NONE)
				{
					TempPath = TempPath.Left(pos + 1) + TEXT("../../");
				}
				else
				{
					while (TempPath.Len() && TempPath.Right(1) != TEXT("/"))
					{
						TempPath.LeftChopInline(1, EAllowShrinking::No);
					}
				}
			}
		}

		Path = FPaths::ConvertRelativePathToFull(TempPath);
		FPaths::RemoveDuplicateSlashes(Path);
	}
	return *Path;
}

TArray<FString> FGenericPlatformMisc::GetAdditionalRootDirectories()
{
	FRWScopeLock Lock(TLazySingleton<FStaticData>::Get().AdditionalRootDirectoriesLock, SLT_ReadOnly);

	return TLazySingleton<FStaticData>::Get().AdditionalRootDirectories;
}


const FString FGenericPlatformMisc::VersionCheckPlatformName()
{
	FString OverrideVersionCheckPlatformName;
	if (!GConfig->GetString(TEXT("PatchConfig"), TEXT("PlatformOverrideName"), OverrideVersionCheckPlatformName, GEngineIni))
	{
		OverrideVersionCheckPlatformName = ANSI_TO_TCHAR(FPlatformProperties::IniPlatformName());
		FString PlatformNameSuffix;
		if (FParse::Value(FCommandLine::Get(), TEXT("PlatformSuffix="), PlatformNameSuffix))
		{
			OverrideVersionCheckPlatformName += PlatformNameSuffix;
		}
	}
	return OverrideVersionCheckPlatformName;
}

void FGenericPlatformMisc::AddAdditionalRootDirectory(const FString& RootDir)
{
	FRWScopeLock Lock(TLazySingleton<FStaticData>::Get().AdditionalRootDirectoriesLock, SLT_Write);

	TArray<FString>& RootDirectories = TLazySingleton<FStaticData>::Get().AdditionalRootDirectories;
	FString NewRootDirectory = RootDir;
	FPaths::MakePlatformFilename(NewRootDirectory);
	RootDirectories.Add(NewRootDirectory);
}

static void MakeEngineDir(FString& OutEngineDir)
{
	// See if we are a root-level project
	FString DefaultEngineDir = "../../../Engine/";
#if PLATFORM_DESKTOP
#if !defined(DISABLE_CWD_CHANGES) || DISABLE_CWD_CHANGES == 0
	FPlatformProcess::SetCurrentWorkingDirectoryToBaseDir();
#endif

	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();

	const TCHAR* BaseDir = FPlatformProcess::BaseDir();

	//@todo. Need to have a define specific for this scenario??
	FString DirToTry = BaseDir / DefaultEngineDir / TEXT("Binaries");
	if (PlatformFile.DirectoryExists(*DirToTry))
	{
		OutEngineDir = MoveTemp(DefaultEngineDir);
		return;
	}

	if (GForeignEngineDir)
	{
		DirToTry = FString(GForeignEngineDir) / TEXT("Binaries");
		if (PlatformFile.DirectoryExists(*DirToTry))
		{
			OutEngineDir = GForeignEngineDir;
			return;
		}
	}

	// Temporary work-around for legacy dependency on ../../../ (re Lightmass)
	UE_LOGF(LogGenericPlatformMisc, Warning, "Failed to determine engine directory: Defaulting to %ls", *OutEngineDir);
#endif

	OutEngineDir = MoveTemp(DefaultEngineDir);
}

const TCHAR* FGenericPlatformMisc::EngineDir()
{
	FString& EngineDirectory = TLazySingleton<FStaticData>::Get().EngineDirectory;
	if (EngineDirectory.Len() == 0)
	{
		MakeEngineDir(EngineDirectory);
	}
	return *EngineDirectory;
}

void FGenericPlatformMisc::CacheLaunchDir()
{
	FString& LaunchDir = TLazySingleton<FStaticData>::Get().LaunchDir;

	// we can only cache this ONCE
	if (LaunchDir.Len() != 0)
	{
		return;
	}
	
	LaunchDir = FPlatformProcess::GetCurrentWorkingDirectory() + TEXT("/");
}

const TCHAR* FGenericPlatformMisc::LaunchDir()
{
	return *TLazySingleton<FStaticData>::Get().LaunchDir;
}


const TCHAR* FGenericPlatformMisc::GetNullRHIShaderFormat()
{
	if (FParse::Param(FCommandLine::Get(), TEXT("sm5")))
	{
		return TEXT("PCD3D_SM5");
	}
	else
	{
		return TEXT("PCD3D_SM6");
	}
}

IPlatformChunkInstall* FGenericPlatformMisc::GetPlatformChunkInstall()
{
	static FGenericPlatformChunkInstall Singleton;
	return &Singleton;
}

IPlatformHostCommunication& FGenericPlatformMisc::GetPlatformHostCommunication()
{
	static FPlatformHostCommunicationAutoInit<FGenericPlatformHostCommunication> Singleton;
	return Singleton;
}

bool FGenericPlatformMisc::IsPreferredInputDevice(const TCHAR* DeviceName)
{
	// If given a null treat it as always being enabled.
	// This ensures that users who have written their own IInputDeviceModule code will
	// continue working and they can add support for this feature if they want to.
	if (!DeviceName)
	{
		return true;
	}
	
	bool bIsPreferredAPICheckEnabled = false;
	
	GConfig->GetBool(TEXT("/Script/Engine.InputSettings"), TEXT("bEnablePreferredInputAPIPreferences"), OUT bIsPreferredAPICheckEnabled, GInputIni);
	
	// If the project setting is not enabled, always return true because we want every API to be enabled
	if (!bIsPreferredAPICheckEnabled)
	{
		return true;
	}

	const FString CurrentPreferredInputAPIs = FGenericPlatformMisc::GetCurrentPreferredInputDevices();

	UE_CLOGF(CurrentPreferredInputAPIs.IsEmpty(), LogGenericPlatformMisc, Warning, "bEnablePreferredInputAPIPreferences is true, but no default preferred APIs were provided. Please fill out DefaultPreferredInputAPIList in the input settings for this to have any affect");
	
	// If the given device is contained in our comma-seperated string value, then it is preferred
	return CurrentPreferredInputAPIs.Contains(DeviceName);
}

void FGenericPlatformMisc::SetPreferredInputDevices(const TCHAR* DeviceNames)
{
	GConfig->SetString(TEXT("GamepadAPIPreference"), TEXT("PreferredAPI"), DeviceNames, GGameUserSettingsIni);

	UE_LOGF(LogGenericPlatformMisc, Log, "Changed preferred gamepad API to '%ls'", DeviceNames);
}

void FGenericPlatformMisc::ResetPreferredInputDevicesToDefault()
{
	// Clear the user set value
	GConfig->RemoveKey(TEXT("GamepadAPIPreference"), TEXT("PreferredAPI"), GGameUserSettingsIni);

	// Set it to the default specified in the project if there is one
	FString DefaultPreferredAPIs = TEXT("");
	const bool bHasDefaultPreferredList = GConfig->GetString(TEXT("/Script/Engine.InputSettings"), TEXT("DefaultPreferredInputAPIList"), DefaultPreferredAPIs, GInputIni);
	if (bHasDefaultPreferredList && !DefaultPreferredAPIs.IsEmpty())
	{
		SetPreferredInputDevices(*DefaultPreferredAPIs);
	}

	UE_LOGF(LogGenericPlatformMisc, Log, "Changed preferred gamepad API to '%ls'", *DefaultPreferredAPIs);
}

FString FGenericPlatformMisc::GetCurrentPreferredInputDevices()
{
	// If the API check is enabled, we want to look at the game's user preference INI and see what the use has selected for their API 
	FString CurrentPreferredAPIs = TEXT("");

#if WITH_EDITOR
	// First, check the editor preference for overriden API preferences here allowing for easier testing
	if (GConfig->GetString(TEXT("/Script/UnrealEd.LevelEditorPlaySettings"), TEXT("PreferredInputAPIOverride"), OUT CurrentPreferredAPIs, GEditorPerProjectIni) && !CurrentPreferredAPIs.IsEmpty())
	{
		return CurrentPreferredAPIs;
	}
#endif

	// Next, check if the user has set a preference yet. If they have, use that
	if (GConfig->GetString(TEXT("GamepadAPIPreference"), TEXT("PreferredAPI"), OUT CurrentPreferredAPIs, GGameUserSettingsIni))
	{
		return CurrentPreferredAPIs;
	}
		
	// Lastly, use the project's default preferred list if there is one.
	if (GConfig->GetString(TEXT("/Script/Engine.InputSettings"), TEXT("DefaultPreferredInputAPIList"), OUT CurrentPreferredAPIs, GInputIni))
	{
		return CurrentPreferredAPIs;
	}
		
	return CurrentPreferredAPIs;
}

void GenericPlatformMisc_GetProjectFilePathProjectDir(FString& OutGameDir)
{
	// Here we derive the game path from the project file location.
	FString BasePath = FPaths::GetPath(FPaths::GetProjectFilePath());
	FPaths::NormalizeFilename(BasePath);
	BasePath = FFileManagerGeneric::DefaultConvertToRelativePath(*BasePath);
	if(!BasePath.EndsWith("/")) BasePath += TEXT("/");
	OutGameDir = BasePath;
}

const TCHAR* FGenericPlatformMisc::ProjectDir()
{
	FString& ProjectDir = TLazySingleton<FStaticData>::Get().ProjectDir;

	// always call this at ProjectDir() time and cache it (internally) because there could be pak file interaction that can disable
	// ini file loading
	bool bIsStaged = FPaths::IsStaged();

	// track if last time we called this function the .ini was ready and had fixed the GameName case
	static bool bWasIniReady = false;
	bool bIsIniReady = GConfig && GConfig->IsReadyForUse();
	if (bWasIniReady != bIsIniReady)
	{
		ProjectDir.Reset();
		bWasIniReady = bIsIniReady;
	}

	// track if last time we called this function the project file path was set
	static bool bWasProjectFilePathReady = false;
	if (!bWasProjectFilePathReady && FPaths::IsProjectFilePathSet())
	{
		ProjectDir.Reset();
		bWasProjectFilePathReady = true;
	}

	// try using the override game dir if it exists, which will override all below logic
	if (ProjectDir.Len() == 0)
	{
		ProjectDir.Reserve(FPlatformMisc::GetMaxPathLength());
		ProjectDir = GOverrideProjectDir;
	}

	if (ProjectDir.Len() == 0)
	{
		ProjectDir.Reserve(FPlatformMisc::GetMaxPathLength());
		if (FPlatformProperties::IsProgram())
		{
			if (bIsStaged)
			{
				// if staged, use the remapped location
				ProjectDir = FString::Printf(TEXT("../../../%s/"), FApp::GetProjectName());
			}
			else
			{
				// monolithic, game-agnostic executables
				ProjectDir = FString::Printf(TEXT("../../../Engine/Programs/%s/"), FApp::GetProjectName());
			}
		}
		else
		{
			if (FPaths::IsProjectFilePathSet())
			{
				GenericPlatformMisc_GetProjectFilePathProjectDir(ProjectDir);
			}
			else if ( FApp::HasProjectName() )
			{
				if (FPlatformProperties::IsMonolithicBuild() == false)
				{
					// No game project file, but has a game name, use the game folder next to the working directory
					ProjectDir = FString::Printf(TEXT("../../../%s/"), FApp::GetProjectName());
					FString GameBinariesDir = ProjectDir / TEXT("Binaries/");
					if (FPlatformFileManager::Get().GetPlatformFile().DirectoryExists(*GameBinariesDir) == false)
					{
						// The game binaries folder was *not* found
						// 
						FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Failed to find game directory: %s\n"), *ProjectDir);

						// Use the uprojectdirs
						FString GameProjectFile = FUProjectDictionary::GetDefault().GetRelativeProjectPathForGame(FApp::GetProjectName(), FPlatformProcess::BaseDir());
						if (GameProjectFile.IsEmpty() == false)
						{
							// We found a project folder for the game
							FPaths::SetProjectFilePath(GameProjectFile);
							ProjectDir = FPaths::GetPath(GameProjectFile);
						}
					}
				}
				else
				{
#if !PLATFORM_PROJECT_DIR_RELATIVE_TO_EXECUTABLE
					ProjectDir = FString::Printf(TEXT("../../../%s/"), FApp::GetProjectName());
#else
					// This assumes the game executable is in <GAME>/Binaries/<PLATFORM>
					ProjectDir = TEXT("../../");

					// Determine a relative path that includes the game folder itself, if possible...
					FString LocalBaseDir = FString(FPlatformProcess::BaseDir());
					FString LocalRootDir = FPaths::RootDir();
					FString BaseToRoot = LocalRootDir;
					FPaths::MakePathRelativeTo(BaseToRoot, *LocalBaseDir);
					FString LocalProjectDir = LocalBaseDir / TEXT("../../");
					FPaths::CollapseRelativeDirectories(LocalProjectDir);
					FPaths::MakePathRelativeTo(LocalProjectDir, *(FPaths::RootDir()));
					LocalProjectDir = BaseToRoot / LocalProjectDir;
					if (LocalProjectDir.EndsWith(TEXT("/")) == false)
					{
						LocalProjectDir += TEXT("/");
					}

					FString CheckLocal = FPaths::ConvertRelativePathToFull(LocalProjectDir);
					FString CheckGame = FPaths::ConvertRelativePathToFull(ProjectDir);
					if (CheckLocal == CheckGame)
					{
						ProjectDir = LocalProjectDir;
					}
#endif
				}
			}
			else
			{
				// Get a writable engine directory
				ProjectDir = FPaths::EngineUserDir();
				FPaths::NormalizeFilename(ProjectDir);
				ProjectDir = FFileManagerGeneric::DefaultConvertToRelativePath(*ProjectDir);
			}
		}
		if (!ProjectDir.EndsWith(TEXT("/")))
		{
			ProjectDir += TEXT("/");
		}
	}

	return *ProjectDir;
}

bool FGenericPlatformMisc::GetEngineAndProjectAbsoluteDirsFromExecutable(FString& OutProjectDir, FString& OutEngineDir)
{
	IPlatformFile& PlatformFile = IPlatformFile::GetPlatformPhysical();

	FString ExecutableDir = FPaths::GetPath(FPlatformProcess::ExecutablePath());
	FPaths::NormalizeFilename(ExecutableDir);

	FString AbsoluteEngineDir = FPaths::Combine(ExecutableDir, TEXT("../.."));
	FPaths::CollapseRelativeDirectories(AbsoluteEngineDir);
	FString EngineBinariesDir = AbsoluteEngineDir / TEXT("Binaries");
	if (!PlatformFile.DirectoryExists(*EngineBinariesDir))
	{
		return false;
	}

	OutEngineDir = AbsoluteEngineDir;


	// First try the most common placement of projects
	FString AbsoluteProjectDir = FPaths::Combine(ExecutableDir, TEXT("../../.."), FApp::GetProjectName());
	FPaths::CollapseRelativeDirectories(AbsoluteProjectDir);

	FString ProjectBinariesDir = AbsoluteProjectDir / TEXT("Binaries");
	if (PlatformFile.DirectoryExists(*ProjectBinariesDir))
	{
		OutProjectDir = AbsoluteProjectDir;
		return true;
	}

	// The game binaries folder was *not* found
	// 
	FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Failed to find game directory: %s\n"), *ProjectBinariesDir);

	FString RootDir = AbsoluteEngineDir / TEXT("..");
	FPaths::CollapseRelativeDirectories(RootDir);

	// Use the uprojectdirs
	FUProjectDictionary Dict(RootDir);
	FString GameProjectFile = Dict.GetProjectPathForGame(FApp::GetProjectName());
	if (GameProjectFile.IsEmpty())
	{
		return false;
	}
	
	OutProjectDir = FPaths::GetPath(GameProjectFile);

	return true;
}

FString FGenericPlatformMisc::CloudDir()
{
	return FPaths::ProjectSavedDir() + TEXT("Cloud/");
}

const TCHAR* FGenericPlatformMisc::GamePersistentDownloadDir()
{
	FString& GamePersistentDownloadDir = TLazySingleton<FStaticData>::Get().GamePersistentDownloadDir;

	if (GamePersistentDownloadDir.Len() == 0)
	{
		GamePersistentDownloadDir = FPaths::ProjectSavedDir() / TEXT("PersistentDownloadDir");
	}
	return *GamePersistentDownloadDir;
}

const TCHAR* FGenericPlatformMisc::GameTemporaryDownloadDir()
{
	return nullptr;
}

const TCHAR* FGenericPlatformMisc::GeneratedConfigDir()
{
	static FString Dir = FPaths::ProjectSavedDir() / TEXT("Config/");
	return *Dir;
}

const TCHAR* FGenericPlatformMisc::GetUBTPlatform()
{
	return TEXT(UE_STRINGIZE(UBT_COMPILED_PLATFORM));
}

const TCHAR* FGenericPlatformMisc::GetUBTTarget()
{
	return TEXT(UE_STRINGIZE(UBT_COMPILED_TARGET));
}

const TCHAR* FGenericPlatformMisc::GetUBTArchitecture()
{
	return TEXT(UE_STRINGIZE(UBT_COMPILED_ARCHITECTURE));
}


using FUBTTargetNameArrayType = TCHAR[128];

#if PLATFORM_TCHAR_IS_UTF8CHAR

	// We can't initialize a sized UTF8CHAR static array with a UTF8TEXT until we have char8_t in C++20,
	// so we use a constructor to copy the value in.
	static FUBTTargetNameArrayType& GetStaticUBTTargetName()
	{
		static struct FInitializer
		{
			FInitializer()
			{
				/** The name of the UBT target that the current executable was built from. Defaults to the UE default target for this type to make content only projects work,
					but will be overridden by the primary game module if it exists */
				FCString::Strcpy(UBTTargetName, TEXT("Unreal" UE_STRINGIZE(UBT_COMPILED_TARGET)));
			}

			TCHAR UBTTargetName[128];
		} Initializer;

		return Initializer.UBTTargetName;
	}

#else

	static FUBTTargetNameArrayType& GetStaticUBTTargetName()
	{
		/** The name of the UBT target that the current executable was built from. Defaults to the UE default target for this type to make content only projects work,
			but will be overridden by the primary game module if it exists */
		static FUBTTargetNameArrayType GUBTTargetName = TEXT("Unreal" UE_STRINGIZE(UBT_COMPILED_TARGET));

		return GUBTTargetName;
	}

#endif

void FGenericPlatformMisc::SetUBTTargetName(const TCHAR* InTargetName)
{
	FUBTTargetNameArrayType& UBTTargetName = GetStaticUBTTargetName();
	check(FCString::Strlen(InTargetName) < (UE_ARRAY_COUNT(UBTTargetName) - 1));
	FCString::Strcpy(UBTTargetName, InTargetName);
}

const TCHAR* FGenericPlatformMisc::GetUBTTargetName()
{
	return GetStaticUBTTargetName();
}

const TCHAR* FGenericPlatformMisc::GetDefaultDeviceProfileName()
{
	return TEXT("Default");
}

float FGenericPlatformMisc::GetDeviceTemperatureLevel()
{
	return -1.0f;
}

float FGenericPlatformMisc::GetDeviceTemperature()
{ 
	return -1.0f; 
}

EDeviceThermalState FGenericPlatformMisc::GetDeviceThermalState()
{ 
	return EDeviceThermalState::Unsupported; 
}

void FGenericPlatformMisc::SetOverrideProjectDir(const FString& InOverrideDir)
{
	GOverrideProjectDir = InOverrideDir;
}

bool FGenericPlatformMisc::UseRenderThread()
{
	// look for disabling commandline options (-onethread is old-school, here for compatibility with people's brains)
	if (FParse::Param(FCommandLine::Get(), TEXT("norenderthread")) || FParse::Param(FCommandLine::Get(), TEXT("onethread")))
	{
		return false;
	}

	// single core devices shouldn't use it (unless that platform overrides this function - maybe RT could be required?)
	if (FPlatformMisc::NumberOfCoresIncludingHyperthreads() < 2)
	{
		return false;
	}

	// if the platform doesn't allow threading at all, we really can't use it
	if (FPlatformProcess::SupportsMultithreading() == false)
	{
		return false;
	}

	// dedicated servers should not use a rendering thread
	if (IsRunningDedicatedServer())
	{
		return false;
	}


#if ENABLE_LOW_LEVEL_MEM_TRACKER
	// disable rendering thread when LLM wants to so that memory is attributer better
	if (FLowLevelMemTracker::Get().ShouldReduceThreads())
	{
		return false;
	}
#endif

	// allow if not overridden
	return true;
}

bool FGenericPlatformMisc::AllowThreadHeartBeat()
{
	static bool bHeartbeat = !FParse::Param(FCommandLine::Get(), TEXT("noheartbeatthread"));
	return bHeartbeat;
}

int32 FGenericPlatformMisc::NumberOfCores()
{
	return 1;
}

int32 FGenericPlatformMisc::NumberOfCoresIncludingHyperthreads()
{
	return FPlatformMisc::NumberOfCores();
}

int32 FGenericPlatformMisc::GetExternalAppMaxPathLength()
{
	return FPlatformMisc::GetMaxPathLength();
}

void FGenericPlatformMisc::GetConfiguredCoreLimits(int32 PlatformNumPhysicalCores, int32 PlatformNumLogicalCores,
	bool& bOutFullyInitialized, int32& OutPhysicalCoreLimit, int32& OutLogicalCoreLimit,
	bool& bOutSetPhysicalCountToLogicalCount)
{
	// If CommandLine is not yet initialized, silently return default values. Callers will need to handle calling again.
	if (!FCommandLine::IsInitialized())
	{
		bOutFullyInitialized = false;
		OutPhysicalCoreLimit = 0;
		OutLogicalCoreLimit = 0;
		bOutSetPhysicalCountToLogicalCount = false;
		return;
	}

	int32 PhysicalCoreLimit = 0;
	int32 LogicalCoreLimit = 0;
	int32 LegacyCoreLimit = 0;
	bool bSetPhysicalCountToLogicalCount = false;

	const TCHAR* CommandLine = FCommandLine::Get();
	FParse::Value(CommandLine, TEXT("-physicalcorelimit="), PhysicalCoreLimit); // DEPRECATION_WARNING: physicalcorelimit is experimental and may be changed in a future release without deprecation
	FParse::Value(CommandLine, TEXT("-corelimit="), LegacyCoreLimit);
	bSetPhysicalCountToLogicalCount = FParse::Param(CommandLine, TEXT("usehyperthreading"));
	if (bSetPhysicalCountToLogicalCount)
	{
		LogicalCoreLimit = PhysicalCoreLimit;
	}
	else
	{
		LogicalCoreLimit = PlatformNumPhysicalCores > 0 ?
			(PhysicalCoreLimit * PlatformNumLogicalCores) / PlatformNumPhysicalCores :
			PhysicalCoreLimit;
	}
	if (LegacyCoreLimit > 0)
	{
		PhysicalCoreLimit = PhysicalCoreLimit == 0 ? LegacyCoreLimit : FMath::Min(PhysicalCoreLimit, LegacyCoreLimit);
		LogicalCoreLimit = LogicalCoreLimit == 0 ? LegacyCoreLimit : FMath::Min(LogicalCoreLimit, LegacyCoreLimit);
	}

#if USING_INSTRUMENTATION
	// Limit the number of threads as we have a limit of 256 context id in the shadow memory structure.
	constexpr int32 InstrumentationCoreLimit = 32;
	PhysicalCoreLimit = PhysicalCoreLimit == 0 ? InstrumentationCoreLimit : FMath::Min(PhysicalCoreLimit, InstrumentationCoreLimit);
	LogicalCoreLimit  = LogicalCoreLimit  == 0 ? InstrumentationCoreLimit : FMath::Min(LogicalCoreLimit,  InstrumentationCoreLimit);
#endif

	bOutFullyInitialized = true;
	OutPhysicalCoreLimit = PhysicalCoreLimit;
	OutLogicalCoreLimit = LogicalCoreLimit;
	bOutSetPhysicalCountToLogicalCount = bSetPhysicalCountToLogicalCount;
}

FProcessorGroupDesc InternalGetProcessorGroupDesc()
{
	FProcessorGroupDesc Desc;
	Desc.NumProcessorGroups = 1;
	memset(Desc.ThreadAffinities, 0xFF, sizeof(Desc.ThreadAffinities));
	return Desc;
}

const FProcessorGroupDesc& FGenericPlatformMisc::GetProcessorGroupDesc()
{
	static FProcessorGroupDesc Desc = InternalGetProcessorGroupDesc();
	return Desc;
}

int32 FGenericPlatformMisc::NumberOfWorkerThreadsToSpawn()
{
	static int32 MaxGameThreads = 4;

	int32 MaxThreads = INT32_MAX;

	int32 NumberOfCores = FPlatformMisc::NumberOfCores();
	int32 MaxWorkerThreadsWanted = (IsRunningGame() || IsRunningDedicatedServer() || IsRunningClientOnly()) ? MaxGameThreads : MaxThreads;
	// need to spawn at least two worker thread (see FTaskGraphImplementation)
	return FMath::Max(FMath::Min(NumberOfCores - 1, MaxWorkerThreadsWanted), 2);
}

int32 FGenericPlatformMisc::NumberOfIOWorkerThreadsToSpawn()
{
	return 4;
}

void FGenericPlatformMisc::GetValidTargetPlatforms(class TArray<class FString>& TargetPlatformNames)
{
	// by default, just return the running PlatformName as the only TargetPlatform we support
	TargetPlatformNames.Add(FPlatformProperties::PlatformName());
}

FPlatformUserId FGenericPlatformMisc::GetPlatformUserForUserIndex(int32 LocalUserIndex)
{
	// These currently map 1:1 but that could change with the input system rework
	return FPlatformUserId::CreateFromInternalId(LocalUserIndex);
}

int32 FGenericPlatformMisc::GetUserIndexForPlatformUser(FPlatformUserId PlatformUser)
{
	return PlatformUser.GetInternalId();
}

TArray<uint8> FGenericPlatformMisc::GetSystemFontBytes()
{
	return TArray<uint8>();
}

const TCHAR* FGenericPlatformMisc::GetDefaultPathSeparator()
{
	return TEXT( "/" );
}

bool FGenericPlatformMisc::GetSHA256Signature(const void* Data, uint32 ByteSize, FSHA256Signature& OutSignature)
{
	checkf(false, TEXT("No SHA256 Platform implementation"));
	FMemory::Memzero(OutSignature.Signature);
	return false;
}

FString FGenericPlatformMisc::GetDefaultLanguage()
{
	return FPlatformMisc::GetDefaultLocale();
}

FString FGenericPlatformMisc::GetDefaultLocale()
{
#if UE_ENABLE_ICU
	icu::Locale ICUDefaultLocale = icu::Locale::getDefault();
	return FString(ICUDefaultLocale.getName());
#else
	return TEXT("en");
#endif
}

FString FGenericPlatformMisc::GetTimeZoneId()
{
	// ICU will calculate this correctly for most platforms (if enabled)
	return FString();
}

#if DO_ENSURE
namespace GenericPlatformMisc
{
	/** Chances for handling an ensure (0.0 - never, 1.0 - always). */
	float GEnsureChance = 1.0f;

	/** Checks if we ever updated ensure settings. */
	bool GEnsureSettingsEverUpdated = false;
}

UE_AUTORTFM_ALWAYS_OPEN
bool FGenericPlatformMisc::IsEnsureAllowed()
{
	// not all targets call FEngineLoop::Tick() or we might be here early
	if (!GenericPlatformMisc::GEnsureSettingsEverUpdated)
	{
		FPlatformMisc::UpdateHotfixableEnsureSettings();
	}

	// using random makes it less deterministic between runs and multiple processes
	return FMath::FRand() < GenericPlatformMisc::GEnsureChance;
}

void FGenericPlatformMisc::UpdateHotfixableEnsureSettings()
{
	// config (which is hotfixable) makes priority over the commandline
	float HandleEnsurePercentInConfig = 100.0f;
	if (GConfig && GConfig->GetFloat(TEXT("Core.System"), TEXT("HandleEnsurePercent"), HandleEnsurePercentInConfig, GEngineIni))
	{
		GenericPlatformMisc::GEnsureChance = HandleEnsurePercentInConfig / 100.0f;
	}
	else
	{
		float HandleEnsurePercentOnCmdLine = 100.0f;
		if (FCommandLine::IsInitialized() && FParse::Value(FCommandLine::Get(), TEXT("handleensurepercent="), HandleEnsurePercentOnCmdLine))
		{
			GenericPlatformMisc::GEnsureChance = HandleEnsurePercentOnCmdLine / 100.0f;
		}
	}

	// to compensate for FRand() being able to return 1.0 (argh!), extra check for 100 
	if (GenericPlatformMisc::GEnsureChance >= 1.00f)
	{
		GenericPlatformMisc::GEnsureChance = 1.01f;
	}

	GenericPlatformMisc::GEnsureSettingsEverUpdated = true;
}
#endif // #if DO_ENSURE

void FGenericPlatformMisc::TickHotfixables()
{
	UpdateHotfixableEnsureSettings();
}

FText FGenericPlatformMisc::GetFileManagerName()
{
	return NSLOCTEXT("GenericPlatform", "FileManagerName", "File Manager");
}

bool FGenericPlatformMisc::IsDeviceGamingHandheld()
{
	return false;
}

bool FGenericPlatformMisc::IsRunningOnBattery()
{
	return false;
}

EDeviceScreenOrientation FGenericPlatformMisc::GetDeviceOrientation()
{
	return EDeviceScreenOrientation::Unknown;
}

EDeviceScreenOrientation FGenericPlatformMisc::GetAllowedDeviceOrientation()
{
	return AllowedDeviceOrientation;
}

void FGenericPlatformMisc::SetAllowedDeviceOrientation(EDeviceScreenOrientation NewAllowedDeviceOrientation)
{
	AllowedDeviceOrientation = NewAllowedDeviceOrientation;
}

int32 FGenericPlatformMisc::GetDeviceVolume()
{
	return -1;
}

FGuid FGenericPlatformMisc::GetMachineId()
{
	static FGuid MachineId;
	FString MachineIdString;

	// Check to see if we already have a valid machine ID to use
	if( !MachineId.IsValid() && (!FPlatformMisc::GetStoredValue( TEXT( "Epic Games" ), TEXT( "Unreal Engine/Identifiers" ), TEXT( "MachineId" ), MachineIdString ) || !FGuid::Parse( MachineIdString, MachineId )) )
	{
		// No valid machine ID, generate and save a new one
		MachineId = FGuid::NewGuid();
		MachineIdString = MachineId.ToString( EGuidFormats::Digits );

		if( !FPlatformMisc::SetStoredValue( TEXT( "Epic Games" ), TEXT( "Unreal Engine/Identifiers" ), TEXT( "MachineId" ), MachineIdString ) )
		{
			// Failed to persist the machine ID - reset it to zero to avoid returning a transient value
			MachineId = FGuid();
		}
	}

	return MachineId;
}

FString FGenericPlatformMisc::GetDeviceTag()
	{
		FString DeviceTag;
		FParse::Value( FCommandLine::Get(), TEXT("DeviceTag="), DeviceTag );

		return DeviceTag;
	}

FString FGenericPlatformMisc::GetLoginId()
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FGuid Id = FPlatformMisc::GetMachineId();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	// force an empty string if we cannot determine an ID.
	if (Id == FGuid())
	{
		return FString();
	}
	return Id.ToString(EGuidFormats::Digits).ToLower();
}


FString FGenericPlatformMisc::GetEpicAccountId()
{
	FString AccountId;
	FPlatformMisc::GetStoredValue( TEXT( "Epic Games" ), TEXT( "Unreal Engine/Identifiers" ), TEXT( "AccountId" ), AccountId );
	return AccountId;
}

EConvertibleLaptopMode FGenericPlatformMisc::GetConvertibleLaptopMode()
{
	return EConvertibleLaptopMode::NotSupported;
}

const TCHAR* FGenericPlatformMisc::GetEngineMode()
{
	return	
		IsRunningCommandlet() ? TEXT( "Commandlet" ) :
		GIsEditor ? TEXT( "Editor" ) :
		IsRunningDedicatedServer() ? TEXT( "Server" ) :
		TEXT( "Game" );
}

TArray<FString> FGenericPlatformMisc::GetPreferredLanguages()
{
	// Determine what out current culture is, and grab the most appropriate set of subtitles for it
	FInternationalization& Internationalization = FInternationalization::Get();

	TArray<FString> PrioritizedCultureNames = Internationalization.GetPrioritizedCultureNames(Internationalization.GetCurrentCulture()->GetName());
	return PrioritizedCultureNames;
}

FString FGenericPlatformMisc::GetLocalCurrencyCode()
{
	// not implemented by default
	return FString();
}

FString FGenericPlatformMisc::GetLocalCurrencySymbol()
{
	// not implemented by default
	return FString();
}

void FGenericPlatformMisc::PlatformPreInit()
{
	FGenericCrashContext::Initialize();
}

FString FGenericPlatformMisc::GetOperatingSystemId()
{
	// not implemented by default.
	return FString();
}

void FGenericPlatformMisc::RegisterForRemoteNotifications()
{
	// not implemented by default
}

bool FGenericPlatformMisc::IsRegisteredForRemoteNotifications()
{
	// not implemented by default
	return false;
}

void FGenericPlatformMisc::UnregisterForRemoteNotifications()
{
	// not implemented by default
}

FDelegateHandle FGenericPlatformMisc::RegisterAudioFocusChangedEventListener(const FOnAudioFocusChangedDelegate Callback)
{
	// not implemented by default
	return FDelegateHandle();
}

void FGenericPlatformMisc::UnregisterAudioFocusChangedEventListener(const FDelegateHandle DelegateHandle)
{
	// not implemented by default
}

FString FGenericPlatformMisc::GetPendingActivationProtocol()
{
	// not implemented by default
	return FString();
}

bool FGenericPlatformMisc::RequestDeviceCheckToken(TFunction<void(const TArray<uint8>&)> QuerySucceededFunc, TFunction<void(const FString&, const FString&)> QueryFailedFunc)
{
	// not implemented by default
	return false;
}

FString FGenericPlatformMisc::LoadTextFileFromPlatformPackage(const FString& RelativePath)
{
	FString Path = RootDir() / RelativePath;
	FString Result;
	if (FFileHelper::LoadFileToString(Result, &FPlatformFileManager::Get().GetPlatformPhysical(), *Path))
	{
		return Result;
	}

	Result.Empty();
	return Result;
}

bool FGenericPlatformMisc::FileExistsInPlatformPackage(const FString& RelativePath)
{
	if (!FPathViews::IsRelativePath(RelativePath))
	{
		UE_LOGF(LogGenericPlatformMisc, Warning, "FileExistsInPlatformPackage: expected a relative path, but received %ls", *RelativePath);
		return false;
	}

	FString Path = RootDir() / RelativePath;
	return IPlatformFile::GetPlatformPhysical().FileExists(*Path);
}

void FGenericPlatformMisc::TearDown()
{
	TLazySingleton<FStaticData>::TearDown();
}

void FGenericPlatformMisc::ParseChunkIdPakchunkIndexMapping(TArray<FString> ChunkIndexMappingData, TMap<int32, int32>& OutMapping)
{
	OutMapping.Empty();

	const TCHAR* PropertyOldChunkIndex = TEXT("Old=");
	const TCHAR* PropertyNewChunkIndex = TEXT("New=");
	for (FString& Entry : ChunkIndexMappingData)
	{
		// Remove parentheses
		Entry.TrimStartAndEndInline();
		Entry.ReplaceInline(TEXT("("), TEXT(""));
		Entry.ReplaceInline(TEXT(")"), TEXT(""));

		int32 ChunkId = -1;
		int32 PakchunkIndex = -1;
		FParse::Value(*Entry, PropertyOldChunkIndex, ChunkId);
		FParse::Value(*Entry, PropertyNewChunkIndex, PakchunkIndex);

		if (ChunkId != -1 && PakchunkIndex != -1 && ChunkId != PakchunkIndex && !OutMapping.Contains(ChunkId))
		{
			OutMapping.Add(ChunkId, PakchunkIndex);
		}
	}
}

FStringView FGenericPlatformMisc::GetPakFilenamePrefix()
{
	return TEXTVIEW("pakchunk");
}

int32 FGenericPlatformMisc::GetPakchunkIndexFromPakFile(FStringView InFilename)
{
	FStringView ChunkIdentifier(GetPakFilenamePrefix());
	const FString NamedChunkIdentifier(FString(GetPakFilenamePrefix()) + NAMED_PAK_CHUNK_DELIMITER_CHAR);
	FStringView BaseFilename = FPathViews::GetBaseFilename(InFilename);
	int32 ChunkNumber = INDEX_NONE;

	if (FCoreDelegates::PakChunkIdOverrideDelegate.IsBound() && BaseFilename.StartsWith(*NamedChunkIdentifier))
	{
		const int32 StartOfChunkName = NamedChunkIdentifier.Len();
		int32 CharCount = 0;
		while ((CharCount + StartOfChunkName) < BaseFilename.Len() && BaseFilename[StartOfChunkName + CharCount] != NAMED_PAK_CHUNK_DELIMITER_CHAR)
		{
			CharCount++;
		}

		FString ChunkName = FString(BaseFilename.Mid(StartOfChunkName, CharCount));
		int32 ChunkId = FCoreDelegates::PakChunkIdOverrideDelegate.Execute(ChunkName);
		if (ChunkId != INDEX_NONE)
		{
			return ChunkId;
		}
	}
	else if (BaseFilename.StartsWith(ChunkIdentifier))
	{
		const int32 StartOfNumber = ChunkIdentifier.Len();
		int32 DigitCount = 0;
		
		while ((DigitCount + StartOfNumber) < BaseFilename.Len() && FChar::IsDigit(BaseFilename[StartOfNumber + DigitCount]))
		{
			DigitCount++;
		}

		if (DigitCount > 0 && (StartOfNumber + DigitCount) <= BaseFilename.Len())
		{
			// FromString can't take a view
			TStringBuilder<16> ChunkNumberString = WriteToString<16>(BaseFilename.Mid(StartOfNumber, DigitCount));
			TTypeFromString<int32>::FromString(ChunkNumber, *ChunkNumberString);
		}
	}

	return ChunkNumber;
}

bool FGenericPlatformMisc::IsPGOEnabled()
{
	return PLATFORM_COMPILER_OPTIMIZATION_PG != 0;
}

bool FGenericPlatformMisc::IsPGICapableBinary()
{
	return PLATFORM_COMPILER_OPTIMIZATION_PG_PROFILING != 0;
}


bool FGenericPlatformMisc::IsPGIActive()
{
	// by default, assume it enabled from the start in PGI binaries (as is the usual behavior). If a platform provides a way
	// to enable/disable PG data collection runtime, it can override this.
	return FPlatformMisc::IsPGICapableBinary();
}

int FGenericPlatformMisc::GetMobilePropagateAlphaSetting()
{
	static auto* PropagateAlphaCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Mobile.PropagateAlpha"));
	return PropagateAlphaCVar ? PropagateAlphaCVar->GetInt() : 0;
}

void FGenericPlatformMisc::SanitizeSystemWideMutexName(FString& InName)
{
	// Remove all invalid filesystem characters from the provided name but also 
	// remove '.' since we don't want names with . or .. to be interpreted as 
	// paths for platforms that might require handle names to be valid paths
	const FString InvalidChars = FString::Printf(TEXT("%s."), *FPaths::GetInvalidFileSystemChars());

	int32 Index = INDEX_NONE;
	for (TCHAR& Char : InName)
	{
		if (InvalidChars.FindChar(Char, Index))
		{
			Char = TEXT('_');
		}
	}
}

void FGenericPlatformMisc::ShowConsoleWindow()
{
#if !UE_BUILD_SHIPPING
	UE_LOGF(LogGenericPlatformMisc, Log, "Show console is not supported or implemented in current platform");
#endif
}

FDelegateHandle FGenericPlatformMisc::AddNetworkListener(FCoreDelegates::FOnNetworkConnectionChanged::FDelegate&& InNewDelegate)
{
	UE_LOGF(LogGenericPlatformMisc, Warning, "FGenericPlatformMisc::AddNetworkListener not implemented for this platform");

	return FDelegateHandle();
}

bool FGenericPlatformMisc::RemoveNetworkListener(FDelegateHandle Handle)
{
	UE_LOGF(LogGenericPlatformMisc, Warning, "FGenericPlatformMisc::RemoveNetworkListener not implemented for this platform");

	return false;
}

const TMap<FString, FString>& FGenericPlatformMisc::GetConfigRuleVars()
{
	static TMap<FString, FString> Temp;
	return Temp;
}

const FUtf8StringView FGenericPlatformMisc::GetConfigRulesVariableAsUtf8(const FUtf8StringView Key)
{
	return FUtf8StringView();
}

const FString* FGenericPlatformMisc::GetConfigRulesVariable(const FString& Key) 
{
	return FPlatformMisc::GetConfigRuleVars().Find(Key);
}
