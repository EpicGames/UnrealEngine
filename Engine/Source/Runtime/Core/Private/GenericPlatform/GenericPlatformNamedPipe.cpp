// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenericPlatform/GenericPlatformNamedPipe.h"
#include "Containers/UnrealString.h"


#if PLATFORM_SUPPORTS_NAMED_PIPES

FGenericPlatformNamedPipe::FGenericPlatformNamedPipe()
{
	NamePtr = new FString();
}


FGenericPlatformNamedPipe::~FGenericPlatformNamedPipe()
{
	delete NamePtr;
	NamePtr = NULL;
}

#endif	// PLATFORM_SUPPORTS_NAMED_PIPES
