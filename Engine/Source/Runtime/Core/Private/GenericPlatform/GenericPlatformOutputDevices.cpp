// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenericPlatform/GenericPlatformOutputDevices.h"
#include "HAL/PlatformOutputDevices.h"
#include "CoreGlobals.h"
#include "Misc/Parse.h"
#include "Misc/CommandLine.h"
#include "Misc/Paths.h"
#include "Misc/OutputDeviceMemory.h"
#include "Misc/OutputDeviceFile.h"
#include "Misc/OutputDeviceDebug.h"
#include "Misc/OutputDeviceAnsiError.h"
#include "Misc/App.h"
#include "Misc/ScopeLock.h"
#include "HAL/FeedbackContextAnsi.h"
#include "Misc/OutputDeviceConsole.h"

TCHAR FGenericPlatformOutputDevices::CachedAbsoluteFilename[FGenericPlatformOutputDevices::AbsoluteFileNameMaxLength] = { 0 };
FCriticalSection FGenericPlatformOutputDevices::LogFilenameLock;

void FGenericPlatformOutputDevices::SetupOutputDevices()
{
	check(GLog);

	ResetCachedAbsoluteFilename();
	
	// Add the default log device (typically file) unless the commandline says otherwise.
	if (!FParse::Param(FCommandLine::Get(), TEXT("NODEFAULTLOG")))
	{
		GLog->AddOutputDevice(FPlatformOutputDevices::GetLog());
	}

	TArray<FOutputDevice*> ChannelFileOverrides;
	FPlatformOutputDevices::GetPerChannelFileOverrides(ChannelFileOverrides);

	for (FOutputDevice* ChannelFileOverride : ChannelFileOverrides)
	{
		GLog->AddOutputDevice(ChannelFileOverride);
	}

#if !NO_LOGGING
	// if console is enabled add an output device, unless the commandline says otherwise...
	if (GLogConsole && !FParse::Param(FCommandLine::Get(), TEXT("NOCONSOLE")))
	{
		GLog->AddOutputDevice(GLogConsole);
	}
	
#if USE_DEBUG_LOGGING
	// If the platform has a separate debug output channel (e.g. OutputDebugString) then add an output device
	// unless logging is turned off
	if (FPlatformMisc::HasSeparateChannelForDebugOutput() && !FParse::Param(FCommandLine::Get(), TEXT("NODEBUGOUTPUT")))
	{
		GLog->AddOutputDevice(new FOutputDeviceDebug());
	}
#endif // USE_DEBUG_LOGGING
#endif

#if USE_EVENT_LOGGING
	GLog->AddOutputDevice(FPlatformOutputDevices::GetEventLog());
#endif // USE_EVENT_LOGGING
};

void FGenericPlatformOutputDevices::ResetCachedAbsoluteFilename()
{
	FScopeLock ScopeLock(&LogFilenameLock);
	CachedAbsoluteFilename[0] = TEXT('\0');
}

void FGenericPlatformOutputDevices::OnLogFileOpened(const TCHAR* Pathname)
{
	FScopeLock ScopeLock(&LogFilenameLock); // This function can be called on any thread, the first that serialize a log and lazily create the log file.
	FCString::Strncpy(CachedAbsoluteFilename, Pathname, UE_ARRAY_COUNT(CachedAbsoluteFilename));
}

FString FGenericPlatformOutputDevices::GetAbsoluteLogFilename()
{
	FScopeLock ScopeLock(&LogFilenameLock);

	if (CachedAbsoluteFilename[0] == 0)
	{
		FCString::Strncpy(CachedAbsoluteFilename, *FPaths::ProjectLogDir(), UE_ARRAY_COUNT(CachedAbsoluteFilename));
		FString LogFilename;
		const bool bShouldStopOnSeparator = false;
		if (!FParse::Value(FCommandLine::Get(), TEXT("LOG="), LogFilename, bShouldStopOnSeparator)
			&& !FParse::Value(FCommandLine::Get(), TEXT("LogFileName="), LogFilename, bShouldStopOnSeparator))
		{
			if (FParse::Value(FCommandLine::Get(), TEXT("ABSLOG="), LogFilename, bShouldStopOnSeparator))
			{
				CachedAbsoluteFilename[0] = TEXT('\0');
			}
		}

		FString Extension(FPaths::GetExtension(LogFilename));
		if (Extension != TEXT("log") && Extension != TEXT("txt"))
		{
			// Ignoring the specified log filename because it doesn't have a .log extension.
			LogFilename.Empty();
		}

#if defined(UE_CUSTOM_LOG_FILENAME)
		if (LogFilename.Len() == 0)
		{
			LogFilename = UE_CUSTOM_LOG_FILENAME;
			if (LogFilename.Len() != 0)
			{
				LogFilename += TEXT(".log");
			}
		}
#endif

		if (LogFilename.Len() == 0)
		{
			if (FCString::Strlen(FApp::GetProjectName()) != 0)
			{
				LogFilename = FApp::GetProjectName();
			}
			else
			{
				LogFilename = TEXT("Unreal");
			}

			LogFilename += TEXT(".log");
		}

		FCString::StrncatTruncateDest(CachedAbsoluteFilename, UE_ARRAY_COUNT(CachedAbsoluteFilename), *LogFilename);
	}

	return CachedAbsoluteFilename;
}

#ifndef WITH_LOGGING_TO_MEMORY
	#define WITH_LOGGING_TO_MEMORY 0
#endif

class FOutputDevice* FGenericPlatformOutputDevices::GetLog()
{
	static struct FLogOutputDeviceInitializer
	{
		TUniquePtr<FOutputDevice> LogDevice;
		FLogOutputDeviceInitializer()
		{
#if WITH_LOGGING_TO_MEMORY
			if (!LogDevice
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
				&& FParse::Param(FCommandLine::Get(), TEXT("LOGTOMEMORY"))
#else
				&& !FParse::Param(FCommandLine::Get(), TEXT("NOLOGTOMEMORY")) && !FPlatformProperties::IsServerOnly()
#endif
				)
			{
				LogDevice = MakeUnique<FOutputDeviceMemory>();
			}
#endif // WITH_LOGGING_TO_MEMORY
			if (!LogDevice)
			{
#if (!UE_BUILD_SHIPPING) || PRESERVE_LOG_BACKUPS_IN_SHIPPING
				const bool bDisableBackup = false;
#else
				const bool bDisableBackup = true;
#endif
				LogDevice = MakeUnique<FOutputDeviceFile>(nullptr, bDisableBackup, /*bAppendIfExists*/false, /*bCreateWriterLazily*/true, [](const TCHAR* AbsPathname)
				{
					FGenericPlatformOutputDevices::OnLogFileOpened(AbsPathname);
				});
			}
		}

	} Singleton;

	return Singleton.LogDevice.Get();
}

void FGenericPlatformOutputDevices::GetPerChannelFileOverrides(TArray<FOutputDevice*>& OutputDevices)
{
	FString Commands;
	if (FParse::Value(FCommandLine::Get(), TEXT("logcategoryfiles="), Commands))
	{
		Commands = Commands.TrimQuotes();

		TArray<FString> Parts;
		Commands.ParseIntoArray(Parts, TEXT(","), true);

		for (FString Part : Parts)
		{
			FString Filename, CategoriesString;
			if (Part.TrimStartAndEnd().Split(TEXT("="), &Filename, &CategoriesString))
			{
				// do stuff
				FOutputDeviceFile* OutputDevice = new FOutputDeviceFile(*Filename);

				TArray<FString> Categories;
				CategoriesString.ParseIntoArray(Categories, TEXT("+"), true);

				for (FString Category : Categories)
				{
					OutputDevice->IncludeCategory(FName(*Category));
				}

				OutputDevices.Add(OutputDevice);
			}
		}
	}
}

FOutputDeviceError* FGenericPlatformOutputDevices::GetError()
{
	static FOutputDeviceAnsiError Singleton;
	return &Singleton;
}

FFeedbackContext* FGenericPlatformOutputDevices::GetFeedbackContext()
{
	static FFeedbackContextAnsi Singleton;
	return &Singleton;
}
