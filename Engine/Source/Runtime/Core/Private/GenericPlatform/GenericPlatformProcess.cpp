// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenericPlatform/GenericPlatformProcess.h"
#include "Async/Fundamental/Oversubscription.h"
#include "Misc/Timespan.h"
#include "HAL/PlatformProperties.h"
#include "HAL/PlatformProcess.h"
#include "GenericPlatform/GenericPlatformMutex.h"
#include "Logging/LogMacros.h"
#include "CoreGlobals.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/Parse.h"
#include "Misc/SingleThreadEvent.h"
#include "Misc/CommandLine.h"
#include "Misc/CoreDelegates.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Stats/Stats.h"
#include "Stats/ThreadIdleStats.h"
#include "Misc/CoreStats.h"
#include "Misc/EventPool.h"
#include "Misc/EngineVersion.h"
#include "Misc/LazySingleton.h"
#include "Misc/Fork.h"
#include "Misc/URLRequestFilter.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "Async/TaskGraphInterfaces.h"

#ifndef DEFAULT_NO_THREADING
	// By default we assume the platform supports multithreading
	#define DEFAULT_NO_THREADING 0
#endif

#if PLATFORM_HAS_BSD_TIME 
	#include <unistd.h>
	#include <sched.h>
#endif

DEFINE_STAT(STAT_Sleep);
DEFINE_STAT(STAT_EventWait);

bool IProcessSentinel::DoesSentinelFileExist(const TCHAR* Name, FString* OptionalOutContents)
{
	return FPlatformProcess::CreateProcessSentinelObject()->DoesSentinelFileExistImpl(Name, OptionalOutContents);
}

FString IProcessSentinel::MakeFilename(const TCHAR* Name)
{
	return FPaths::ProjectIntermediateDir() + Name + TEXT(".sentinel");
}


void* FGenericPlatformProcess::GetDllHandle( const TCHAR* Filename )
{
	UE_LOGF(LogHAL, Fatal, "FPlatformProcess::GetDllHandle not implemented on this platform");
	return NULL;
}

void FGenericPlatformProcess::FreeDllHandle( void* DllHandle )
{
	UE_LOGF(LogHAL, Fatal, "FPlatformProcess::FreeDllHandle not implemented on this platform");
}

void* FGenericPlatformProcess::GetDllExport( void* DllHandle, const TCHAR* ProcName )
{
	UE_LOGF(LogHAL, Fatal, "FPlatformProcess::GetDllExport not implemented on this platform");
	return NULL;
}

uint32 FGenericPlatformProcess::GetCurrentProcessId()
{
	// for single-process platforms (consoles, etc), just use 0
	return 0;
}

uint32 FGenericPlatformProcess::GetCurrentCoreNumber()
{
	return 0;
}


void FGenericPlatformProcess::SetThreadAffinityMask( uint64 AffinityMask )
{
	// Not implemented cross-platform. Each platform may or may not choose to implement this.
}

void FGenericPlatformProcess::SetThreadPriority(EThreadPriority NewPriority)
{
	// Not implemented cross-platform. Each platform may or may not choose to implement this.
}

uint32 FGenericPlatformProcess::GetStackSize()
{
	return 0;
}

bool FGenericPlatformProcess::ShouldSaveToUserDir()
{
	// default to use the engine/game directories
	return false;
}

const TCHAR* FGenericPlatformProcess::UserDir()
{
	// default to the root directory
	return FPlatformMisc::RootDir();
}

const TCHAR *FGenericPlatformProcess::UserSettingsDir()
{
	// default to the root directory
	return FPlatformMisc::RootDir();
}

const TCHAR *FGenericPlatformProcess::UserTempDir()
{
	// default to the root directory
	return FPlatformMisc::RootDir();
}

const TCHAR *FGenericPlatformProcess::UserHomeDir()
{
    // default to the root directory
    return FPlatformMisc::RootDir();
}

const TCHAR* FGenericPlatformProcess::ApplicationSettingsDir()
{
	// default to the root directory
	return FPlatformMisc::RootDir();
}

FString FGenericPlatformProcess::GetApplicationSettingsDir(const ApplicationSettingsContext& Settings)
{
	// Default to returning the original `ApplicationSettingsDir` if it's unimplemented.
	return ApplicationSettingsDir();
}


const TCHAR* FGenericPlatformProcess::ComputerName()
{
	return TEXT("GenericComputer");
}

const TCHAR* FGenericPlatformProcess::UserName(bool bOnlyAlphaNumeric/* = true*/)
{
	return TEXT("GenericUser");
}

void FGenericPlatformProcess::SetCurrentWorkingDirectoryToBaseDir()
{
#if defined(DISABLE_CWD_CHANGES) && DISABLE_CWD_CHANGES != 0
	check(false);
#else
	// even if we don't set a directory, we should remember the current one so LaunchDir works
	FPlatformMisc::CacheLaunchDir();
#endif
}

FString FGenericPlatformProcess::GetCurrentWorkingDirectory()
{
	return TEXT("");
}


static FString Generic_ShaderDir;

const TCHAR* FGenericPlatformProcess::ShaderDir()
{
	if (Generic_ShaderDir.Len() == 0)
	{
		Generic_ShaderDir = FPaths::Combine(*(FPaths::EngineDir()), TEXT("Shaders"));
	}
	return *Generic_ShaderDir;
}

void FGenericPlatformProcess::SetShaderDir(const TCHAR*Where)
{
	if ((Where != NULL) && (FCString::Strlen(Where) != 0))
	{
		Generic_ShaderDir = Where;
	}
	else
	{
		Generic_ShaderDir = TEXT("");
	}
}

/**
 *	Get the shader working directory
 */
const FString FGenericPlatformProcess::ShaderWorkingDir()
{
	return (FPaths::ProjectIntermediateDir() / TEXT("Shaders/tmp/"));
}

/**
 *	Clean the shader working directory
 */
void FGenericPlatformProcess::CleanShaderWorkingDir()
{
	// Path to the working directory where files are written for multi-threaded compilation
	FString ShaderWorkingDirectory =  FPlatformProcess::ShaderWorkingDir();
	IFileManager::Get().DeleteDirectory(*ShaderWorkingDirectory, false, true);

	FString LegacyShaderWorkingDirectory = FPaths::ProjectIntermediateDir() / TEXT("Shaders/WorkingDirectory/");
	IFileManager::Get().DeleteDirectory(*LegacyShaderWorkingDirectory, false, true);
}

const TCHAR* FGenericPlatformProcess::ExecutablePath()
{
	UE_LOGF(LogHAL, Fatal, "FGenericPlatformProcess::ExecutablePath not implemented on this platform");
	return NULL;
}

const TCHAR* FGenericPlatformProcess::ExecutableName(bool bRemoveExtension)
{
	UE_LOGF(LogHAL, Fatal, "FGenericPlatformProcess::ExecutableName not implemented on this platform");
	return NULL;
}

FString FGenericPlatformProcess::GenerateApplicationPath( const FString& AppName, EBuildConfiguration BuildConfiguration)
{
	UE_LOGF(LogHAL, Fatal, "FGenericPlatformProcess::GenerateApplicationPath not implemented on this platform");
	return FString();
}

const TCHAR* FGenericPlatformProcess::GetArchitectureSuffix()
{
	return nullptr;
}

const TCHAR* FGenericPlatformProcess::GetModulePrefix()
{
	return TEXT("");
}

const TCHAR* FGenericPlatformProcess::GetModuleExtension()
{
	UE_LOGF(LogHAL, Fatal, "FGenericPlatformProcess::GetModuleExtension not implemented on this platform");
	return TEXT("");
}

const TCHAR* FGenericPlatformProcess::GetBinariesSubdirectory()
{
	return TEXT("");
}

const FString FGenericPlatformProcess::GetModulesDirectory()
{
	return FPaths::Combine(*FPaths::EngineDir(), TEXT("Binaries"), FPlatformProcess::GetBinariesSubdirectory());
}

bool FGenericPlatformProcess::ModuleExists(const FString& Filename)
{
	return FPaths::FileExists(Filename);
}

void FGenericPlatformProcess::LaunchURL( const TCHAR* URL, const TCHAR* Parms, FString* Error )
{
	UE_LOGF(LogHAL, Fatal, "FGenericPlatformProcess::LaunchURL not implemented on this platform");
}

bool FGenericPlatformProcess::LaunchURLFiltered(const TCHAR* URL, const TCHAR* Parms, FString* Error, const UE::Core::FURLRequestFilter& Filter)
{
	const bool bAllowedByFilter = Filter.IsRequestAllowed(URL);

	if (bAllowedByFilter)
	{
		FPlatformProcess::LaunchURL(URL, Parms, Error);
	}
	else if (Error)
	{
		*Error = TEXT("URL rejected by filter");
	}

	return bAllowedByFilter;
}

bool FGenericPlatformProcess::CanLaunchURL(const TCHAR* URL)
{
	UE_LOGF(LogHAL, Warning, "FGenericPlatformProcess::CanLaunchURL not implemented on this platform");
	return false;
}

FString FGenericPlatformProcess::GetGameBundleId()
{
	UE_LOGF(LogHAL, Warning, "FGenericPlatformProcess::GetGameBundleId not implemented on this platform");
	return TEXT("");
}

FProcHandle FGenericPlatformProcess::CreateProc( const TCHAR* URL, const TCHAR* Parms, bool bLaunchDetached, bool bLaunchHidden, bool bLaunchReallyHidden, uint32* OutProcessID, int32 PriorityModifier, const TCHAR* OptionalWorkingDirectory, void* PipeWriteChild, void* PipeReadChild )
{
	UE_LOGF(LogHAL, Fatal, "FGenericPlatformProcess::CreateProc not implemented on this platform");
	return FProcHandle();
}

FProcHandle FGenericPlatformProcess::CreateProc( const TCHAR* URL, const TCHAR* Parms, bool bLaunchDetached, bool bLaunchHidden, bool bLaunchReallyHidden, uint32* OutProcessID, int32 PriorityModifier, const TCHAR* OptionalWorkingDirectory, void* PipeWriteChild, void* PipeReadChild, void* PipeStdErrChild )
{
	UE_LOGF(LogHAL, Fatal, "FGenericPlatformProcess::CreateProc with std out/in/err not implemented on this platform");
	return FProcHandle();
}

TTuple<FProcHandle, UE::HAL::EProcessId> FGenericPlatformProcess::CreateProc(const UE::HAL::FProcessStartInfo& StartInfo)
{
	UE_LOGF(LogHAL, Fatal, "FGenericPlatformProcess::CreateProc not implemented on this platform");
	return {};
}

FProcHandle FGenericPlatformProcess::OpenProcess(uint32 ProcessID)
{
	UE_LOGF(LogHAL, Fatal, "FGenericPlatformProcess::OpenProcess not implemented on this platform");
	return FProcHandle();
}

bool FGenericPlatformProcess::IsProcRunning( FProcHandle & ProcessHandle )
{
	UE_LOGF(LogHAL, Fatal, "FGenericPlatformProcess::IsProcRunning not implemented on this platform");
	return false;
}

void FGenericPlatformProcess::WaitForProc( FProcHandle & ProcessHandle )
{
	UE_LOGF(LogHAL, Fatal, "FGenericPlatformProcess::WaitForProc not implemented on this platform");
}

void FGenericPlatformProcess::CloseProc(FProcHandle & ProcessHandle)
{
	UE_LOGF(LogHAL, Fatal, "FGenericPlatformProcess::CloseProc not implemented on this platform");
}

void FGenericPlatformProcess::TerminateProc( FProcHandle & ProcessHandle, bool KillTree )
{
	UE_LOGF(LogHAL, Fatal, "FGenericPlatformProcess::TerminateProc not implemented on this platform");
}

void FGenericPlatformProcess::TerminateProcTreeWithPredicate(
	FProcHandle& ProcessHandle,
	TFunctionRef<bool(uint32 ProcessId, const TCHAR* ApplicationName)> Predicate)
{
	UE_LOGF(LogHAL, Fatal, "FGenericPlatformProcess::TerminateProcTreeWithPredicate not implemented on this platform");
}

FGenericPlatformProcess::EWaitAndForkResult FGenericPlatformProcess::WaitAndFork()
{
	UE_LOGF(LogHAL, Fatal, "FGenericPlatformProcess::WaitAndFork not implemented on this platform");
	return EWaitAndForkResult::Error;
}

bool FGenericPlatformProcess::GetProcReturnCode( FProcHandle & ProcHandle, int32* ReturnCode )
{
	UE_LOGF(LogHAL, Fatal, "FGenericPlatformProcess::GetProcReturnCode not implemented on this platform");
	return false;
}

bool FGenericPlatformProcess::GetApplicationMemoryUsage(uint32 ProcessId, SIZE_T* OutMemoryUsage)
{
	UE_LOGF(LogHAL, Fatal, "FGenericPlatformProcess::GetApplicationMemoryUsage: not implemented on this platform");
	return false;
}

bool FGenericPlatformProcess::IsApplicationRunning( uint32 ProcessId )
{
	UE_LOGF(LogHAL, Fatal, "FGenericPlatformProcess::IsApplicationRunning not implemented on this platform");
	return false;
}

bool FGenericPlatformProcess::IsApplicationRunning( const TCHAR* ProcName )
{
	UE_LOGF(LogHAL, Fatal, "FGenericPlatformProcess::IsApplicationRunning not implemented on this platform");
	return false;
}

FString FGenericPlatformProcess::GetApplicationName( uint32 ProcessId )
{
	UE_LOGF(LogHAL, Fatal, "FGenericPlatformProcess::GetApplicationName not implemented on this platform");
	return FString();
}

bool FGenericPlatformProcess::ExecProcess(const TCHAR* URL, const TCHAR* Params, int32* OutReturnCode, FString* OutStdOut, FString* OutStdErr, const TCHAR* OptionalWorkingDirectory, bool bShouldEndWithParentProcess)
{
	UE_LOGF(LogHAL, Fatal, "FGenericPlatformProcess::ExecProcess not implemented on this platform");
	return false;
}

bool FGenericPlatformProcess::ExecElevatedProcess(const TCHAR* URL, const TCHAR* Params, int32* OutReturnCode)
{
	return FPlatformProcess::ExecProcess(URL, Params, OutReturnCode, NULL, NULL);
}

bool FGenericPlatformProcess::LaunchFileInDefaultExternalApplication( const TCHAR* FileName, const TCHAR* Parms, ELaunchVerb::Type Verb, bool bPromptToOpenOnFailure )
{
	UE_LOGF(LogHAL, Fatal, "FGenericPlatformProcess::LaunchFileInDefaultExternalApplication not implemented on this platform");
	return false;
}

void FGenericPlatformProcess::ExploreFolder( const TCHAR* FilePath )
{
	UE_LOGF(LogHAL, Fatal, "FGenericPlatformProcess::ExploreFolder not implemented on this platform");
}



#if PLATFORM_HAS_BSD_TIME

void FGenericPlatformProcess::Sleep( float Seconds )
{
	SCOPE_CYCLE_COUNTER(STAT_Sleep);
	UE::Stats::FThreadIdleStats::FScopeIdle Scope;
	SleepNoStats(Seconds);
}

void FGenericPlatformProcess::SleepNoStats( float Seconds )
{
	const int32 usec = FPlatformMath::TruncToInt(Seconds * 1000000.0f);
	if (usec > 0)
	{
		usleep(usec);
	}
	else
	{
		sched_yield();
	}
}

void FGenericPlatformProcess::SleepInfinite()
{
	// stop this thread forever
	while (true)
	{
		pause();
	}
}

void FGenericPlatformProcess::YieldThread()
{
	sched_yield();
}

#endif // PLATFORM_HAS_BSD_TIME 

void FGenericPlatformProcess::ConditionalSleep(TFunctionRef<bool()> Condition, float SleepTime /*= 0.0f*/)
{
	if (Condition())
	{
		return;
	}

	SCOPE_CYCLE_COUNTER(STAT_Sleep);
	UE::Stats::FThreadIdleStats::FScopeIdle Scope;
	do
	{
		FPlatformProcess::SleepNoStats(SleepTime);
	} while (!Condition());
}

#if PLATFORM_USE_PTHREADS

#include "HAL/PThreadEvent.h"

bool FPThreadEvent::Wait(uint32 WaitTime, const bool bIgnoreThreadIdleStats /*= false*/)
{
	WaitForStats();

	SCOPE_CYCLE_COUNTER(STAT_EventWait);
	CSV_SCOPED_WAIT(WaitTime);
	UE::Stats::FThreadIdleStats::FScopeIdle Scope(bIgnoreThreadIdleStats);

	check(bInitialized);

	struct timeval StartTime;

	// We need to know the start time if we're going to do a timed wait.
	if ( (WaitTime > 0) && (WaitTime != ((uint32)-1)) )  // not polling and not infinite wait.
	{
		gettimeofday(&StartTime, NULL);
	}

	LockEventMutex();

	bool bRetVal = false;

	// loop in case we fall through the Condition signal but someone else claims the event.
	do
	{
		// See what state the event is in...we may not have to wait at all...

		// One thread should be released. We saw it first, so we'll take it.
		if (Triggered == TRIGGERED_ONE)
		{
			Triggered = TRIGGERED_NONE;  // dibs!
			bRetVal = true;
		}

		// manual reset that is still signaled. Every thread goes.
		else if (Triggered == TRIGGERED_ALL)
		{
			bRetVal = true;
		}

		// No event signalled yet.
		else if (WaitTime != 0)  // not just polling, wait on the condition variable.
		{
			WaitingThreads++;
			if (WaitTime == ((uint32)-1)) // infinite wait?
			{
				LowLevelTasks::FOversubscriptionScope _; // Let the scheduler know one of its thread might be waiting.
				int rc = pthread_cond_wait(&Condition, &Mutex);  // unlocks Mutex while blocking...
				check(rc == 0);
			}
			else  // timed wait.
			{
				struct timespec TimeOut;
				const uint32 ms = (StartTime.tv_usec / 1000) + WaitTime;
				TimeOut.tv_sec = StartTime.tv_sec + (ms / 1000);
				TimeOut.tv_nsec = (ms % 1000) * 1000000;  // remainder of milliseconds converted to nanoseconds.
				LowLevelTasks::FOversubscriptionScope _; // Let the scheduler know one of its thread might be waiting.
				int rc = pthread_cond_timedwait(&Condition, &Mutex, &TimeOut);    // unlocks Mutex while blocking...
				check((rc == 0) || (rc == ETIMEDOUT));

				// Update WaitTime and StartTime in case we have to go again...
				struct timeval Now, Difference;
				gettimeofday(&Now, NULL);
				SubtractTimevals(&Now, &StartTime, &Difference);
				const int32 DifferenceMS = ((Difference.tv_sec * 1000) + (Difference.tv_usec / 1000));
				WaitTime = ((DifferenceMS >= WaitTime) ? 0 : (WaitTime - DifferenceMS));
				StartTime = Now;
			}
			WaitingThreads--;
			check(WaitingThreads >= 0);
		}

	} while ((!bRetVal) && (WaitTime != 0));

	UnlockEventMutex();
	return bRetVal;
}

#endif

FEvent* FGenericPlatformProcess::CreateSynchEvent(bool bIsManualReset)
{
#if PLATFORM_USE_PTHREADS
	FEvent* Event = NULL;
	
	// Create fake singlethread events in environments that don't support multithreading
	// For processes that intend to fork: create real events even in the master process since the allocated mutex might get reused by the
	//forked child process when it gets to run in multithread mode.
	const bool bIsMultithread = FPlatformProcess::SupportsMultithreading() || FForkProcessHelper::SupportsMultithreadingPostFork();

	if (bIsMultithread)
	{
		// Allocate the new object
		Event = new FPThreadEvent();
	}
	else
	{
		// Fake event.
		Event = new FSingleThreadEvent();
	}
	// If the internal create fails, delete the instance and return NULL
	if (!Event->Create(bIsManualReset))
	{
		delete Event;
		Event = NULL;
	}
	return Event;
#else
	UE_LOGF(LogHAL, Fatal, "FGenericPlatformProcess::CreateSynchEvent not implemented on this platform");
	return NULL;
#endif
}


FEvent* FGenericPlatformProcess::GetSynchEventFromPool(bool bIsManualReset)
{
	return bIsManualReset
		? TLazySingleton<TEventPool<EEventMode::ManualReset>>::Get().GetEventFromPool()
		: TLazySingleton<TEventPool<EEventMode::AutoReset>>::Get().GetEventFromPool();
}

void FGenericPlatformProcess::FlushPoolSyncEvents()
{
	TLazySingleton<TEventPool<EEventMode::ManualReset>>::Get().EmptyPool();
	TLazySingleton<TEventPool<EEventMode::AutoReset>>::Get().EmptyPool();
}

void FGenericPlatformProcess::ReturnSynchEventToPool(FEvent* Event)
{
	if( !Event )
	{
		return;
	}

	if (Event->IsManualReset())
	{
		TLazySingleton<TEventPool<EEventMode::ManualReset>>::Get().ReturnToPool(Event);
	}
	else
	{
		TLazySingleton<TEventPool<EEventMode::AutoReset>>::Get().ReturnToPool(Event);
	}
}


#if PLATFORM_USE_PTHREADS
	#include "HAL/PThreadRunnableThread.h"
#endif

FRunnableThread* FGenericPlatformProcess::CreateRunnableThread()
{
#if PLATFORM_USE_PTHREADS
	return new FRunnableThreadPThread();
#else
	UE_LOGF(LogHAL, Fatal, "FGenericPlatformProcess::CreateThread not implemented on this platform");
	return NULL;
#endif
}

void FGenericPlatformProcess::ClosePipe( void* ReadPipe, void* WritePipe )
{
	UE_LOGF(LogHAL, Fatal, "FGenericPlatformProcess::ClosePipe not implemented on this platform");
}

bool FGenericPlatformProcess::CreatePipe(void*& ReadPipe, void*& WritePipe, bool bWritePipeLocal)
{
	UE_LOGF(LogHAL, Fatal, "FGenericPlatformProcess::CreatePipe not implemented on this platform");
	return false;
}

FString FGenericPlatformProcess::ReadPipe( void* ReadPipe )
{
	UE_LOGF(LogHAL, Fatal, "FGenericPlatformProcess::ReadPipe not implemented on this platform");
	return FString();
}

bool FGenericPlatformProcess::ReadPipeToArray(void* ReadPipe, TArray<uint8> & Output)
{
	UE_LOGF(LogHAL, Fatal, "FGenericPlatformProcess::ReadPipeToArray not implemented on this platform");
	return false;
}

bool FGenericPlatformProcess::WritePipe(void* WritePipe, const FString& Message, FString* OutWritten)
{
	UE_LOGF(LogHAL, Fatal, "FGenericPlatformProcess::WriteToPipe not implemented on this platform");
	return false;
}

bool FGenericPlatformProcess::WritePipe(void* WritePipe, const uint8* Data, const int32 DataLength, int32* OutDataLength)
{
	UE_LOGF(LogHAL, Fatal, "FGenericPlatformProcess::WriteToPipe not implemented on this platform");
	return false;
}

bool FGenericPlatformProcess::SupportsMultithreading()
{
	if (!FCommandLine::IsInitialized())
	{
		// If we don't know yet -- return the default setting
		return !DEFAULT_NO_THREADING;
	}

#if DEFAULT_NO_THREADING
	static bool bSupportsMultithreading = FParse::Param(FCommandLine::Get(), TEXT("threading"));
#else
	static bool bSupportsMultithreading = !FParse::Param(FCommandLine::Get(), TEXT("nothreading"));
#endif

	return bSupportsMultithreading;
}

FGenericPlatformProcess::FSemaphore::FSemaphore(const FString& InName)
	: FSemaphore(*InName)
{
}


FGenericPlatformProcess::FSemaphore::FSemaphore(const TCHAR* InName) 
{
	FCString::Strncpy(Name, InName, UE_ARRAY_COUNT(Name));
}

FGenericPlatformProcess::FSemaphore* FGenericPlatformProcess::NewInterprocessSynchObject(const FString& Name, bool bCreate, uint32 MaxLocks)
{
	return NewInterprocessSynchObject(*Name, bCreate, MaxLocks);
}

FGenericPlatformProcess::FSemaphore* FGenericPlatformProcess::NewInterprocessSynchObject(const TCHAR* Name, bool bCreate, uint32 MaxLocks)
{
	UE_LOGF(LogHAL, Fatal, "FGenericPlatformProcess::NewInterprocessSynchObject not implemented on this platform");
	return NULL;
}

bool FGenericPlatformProcess::DeleteInterprocessSynchObject(FSemaphore * Object)
{
	UE_LOGF(LogHAL, Fatal, "FGenericPlatformProcess::DeleteInterprocessSynchObject not implemented on this platform");
	return false;
}

bool FGenericPlatformProcess::Daemonize()
{
	UE_LOGF(LogHAL, Fatal, "FGenericPlatformProcess::Daemonize not implemented on this platform");
	return false;
}

bool FGenericPlatformProcess::IsFirstInstance()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return GIsFirstInstance;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void FGenericPlatformProcess::CeaseBeingFirstInstance()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	GIsFirstInstance = false;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

UE::FPlatformSystemWideMutexNotImplemented::FPlatformSystemWideMutexNotImplemented(const FString& Name, FTimespan Timeout)
{
	UE_LOGF(LogHAL, Fatal, "FPlatformSystemWideMutex not implemented on this platform.");
}

void FGenericPlatformProcess::TearDown()
{
	TLazySingleton<TEventPool<EEventMode::AutoReset>>::TearDown();
	TLazySingleton<TEventPool<EEventMode::ManualReset>>::TearDown();
}

ENamedThreads::Type FGenericPlatformProcess::GetDesiredThreadForUObjectReferenceCollector()
{
	return ENamedThreads::AnyThread;
}

void FGenericPlatformProcess::ModifyThreadAssignmentForUObjectReferenceCollector( int32& NumThreads, int32& NumBackgroundThreads, ENamedThreads::Type& NormalThreadName, ENamedThreads::Type& BackgroundThreadName )
{
#if PLATFORM_ANDROID
	// On devices with overridden affinity only HiPri threads can run on big cores
	NormalThreadName = ENamedThreads::AnyHiPriThreadHiPriTask; 
	NumBackgroundThreads = 0; // run on single group
#elif WITH_EDITOR
	// Avoid the ReferenceCollector being slowed down by long running background tasks, async compilation, etc...
	NormalThreadName = ENamedThreads::AnyHiPriThreadHiPriTask;
	BackgroundThreadName = ENamedThreads::AnyHiPriThreadHiPriTask;
#endif
}


class FGenericProcessSentinel : public IProcessSentinel
{
public:

	FGenericProcessSentinel()
	{
		FCoreDelegates::OnHandleSystemError.AddRaw(this, &FGenericProcessSentinel::OnCrash);
		FCoreDelegates::OnShutdownAfterError.AddRaw(this, &FGenericProcessSentinel::OnShutdown);
	}
	
	virtual ~FGenericProcessSentinel()
	{
		Cleanup();
	}
	
	/**
	 * Create the sentinel file
	 */
	virtual bool CreateSentinelFile(const TCHAR* Name, const FString& Contents) override
	{
		SavedFilename = MakeFilename(Name);

		if (IFileManager::Get().FileExists(*SavedFilename))
		{
			if (!IFileManager::Get().Delete(*SavedFilename))
			{
				// attempt to lock it, if we can't, then another instance is running
				UE_LOGF(LogHAL, Error, "Another instance is running, unable to delete Sentinel file %ls", *SavedFilename);
				return false;
			}
		}

		// open the file for write so other processes can't
		IFileHandle* Handle = FPlatformFileManager::Get().GetPlatformFile().OpenWrite(*SavedFilename);
		if (Handle)
		{
			char PidString[16];
			int Len = FCStringAnsi::Sprintf(PidString, "%d\n", FPlatformProcess::GetCurrentProcessId());
			Handle->Write((uint8*)PidString, Len);
			
			FTCHARToUTF8 Convert(*Contents);
			Handle->Write((uint8*)Convert.Get(), Convert.Length());
			Handle->Flush(true);

			return true;
		}
		return false;
	}

private:
		
	/** 
	 * The actual implementation to determine if the sentinel file is exists and is valid (due to a process dying,
	 * the file may actually exist on disk, but isn't a valid file anymore, so a simple FileExists may not be good enough
	 */
	virtual bool DoesSentinelFileExistImpl(const TCHAR* Name, FString* OptionalOutContents=nullptr) override
	{
		FString Filename = MakeFilename(Name);
		FString LocalContents;
		FString* Contents = (OptionalOutContents == nullptr) ? &LocalContents : OptionalOutContents;

		/////////////////////////////////////////////////////////////////////////////////////////////////////////////
		// ALTERNATE, MORE SECURE MODE
		// If the platform can depend on Delete failing while another process as the file open for Write, then we
		// can just attempt to delete it - if it gets deleted, then we know another process doesn't have it open 
		// for write, which means the process died unexpectedly, and we can clean it up like it wasn't there
		// However, not all platforms will block deleting if the file is open for write, so it's not a generic
		// solution. 
		// if (!IFileManager::Get().FileExists(*SavedFilename) || IFileManager::Get().Delete(*SavedFilename))
		// {
		//	return false;
		// }
		/////////////////////////////////////////////////////////////////////////////////////////////////////////////

		// get the PID from the first line
		if (FFileHelper::LoadFileToString(*Contents, *Filename))
		{
			int LineBreak = Contents->Find(TEXT("\n"));
			if (LineBreak > 0)
			{
				FString PidString = Contents->Mid(0, LineBreak);
				int Pid = FCString::Atoi(*PidString);
				*Contents = Contents->Mid(LineBreak + 1);
				
				// is pid running
				FProcHandle Proc = FPlatformProcess::OpenProcess(Pid);
				if (FPlatformProcess::IsProcRunning(Proc))
				{
					return true;
				}
				
				// if not, delete the file to clean up
				IFileManager::Get().Delete(*SavedFilename);
			}
		}

		return false;
	}

	/** Callback functions on crash and shutdown after error */
	void OnCrash()
	{
		Cleanup();
	}
	void OnShutdown()
	{
		Cleanup();
	}

	void Cleanup()
	{
		if (FileHandle.IsValid())
		{
			FileHandle.Reset();
		}
		IFileManager::Get().Delete(*SavedFilename);
	}

	FString SavedFilename;
	TUniquePtr<FArchive> FileHandle;
};


TSharedPtr<IProcessSentinel> FGenericPlatformProcess::CreateProcessSentinelObject()
{
	UE_LOGF(LogHAL, Fatal, "This platform doesn't currently support process sentinels, because the generic one can't guarantee that the file will go away on abnormal termination (ie, killing the process, as opposed to crashing normally). If this is needed, look at FGenericProcessSentinel, it's a good starting point.");
	return TSharedPtr<IProcessSentinel>(new FGenericProcessSentinel());
}

TSharedPtr<IProcessSentinel> FGenericPlatformProcess::CreateProcessSentinel(const TCHAR* Name, const FString& Contents)	
{
	TSharedPtr<IProcessSentinel> Sentinel = FPlatformProcess::CreateProcessSentinelObject();
	
	Sentinel->CreateSentinelFile(Name, Contents);
	return Sentinel;
}
