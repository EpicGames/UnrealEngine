// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenericPlatform/GenericPlatformProperties.h"
#include "HAL/PlatformProperties.h"


// This has to be in a .cpp since it needs to come after the FPlatformProperties class has been typedef'd
const char* FGenericPlatformProperties::IniPlatformName()
{
	return FPlatformProperties::PlatformName();
}
