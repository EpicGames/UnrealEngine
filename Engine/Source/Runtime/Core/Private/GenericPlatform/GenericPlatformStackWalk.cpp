// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenericPlatform/GenericPlatformStackWalk.h"
#include "BuildSettings.h"
#include "HAL/PlatformStackWalk.h"
#include "Misc/ConfigCacheIni.h"
#include "GenericPlatform/GenericPlatformCrashContext.h"

FProgramCounterSymbolInfo::FProgramCounterSymbolInfo() :
	LineNumber( 0 ),
	SymbolDisplacement( 0 ),
	OffsetInModule( 0 ),
	ProgramCounter( 0 )
{
	FCStringAnsi::Strncpy( ModuleName, "", MAX_NAME_LENGTH );
	FCStringAnsi::Strncpy( FunctionName, "", MAX_NAME_LENGTH );
	FCStringAnsi::Strncpy( Filename, "", MAX_NAME_LENGTH );
}

FProgramCounterSymbolInfoEx::FProgramCounterSymbolInfoEx(FString InModuleName, FString InFunctionName, FString InFilename, uint32 InLineNumber, uint64 InSymbolDisplacement, uint64 InOffsetInModule, uint64 InProgramCounter) :
	ModuleName(InModuleName),
	FunctionName(InFunctionName),
	Filename(InFilename),
	LineNumber(InLineNumber),
	SymbolDisplacement(InSymbolDisplacement),
	OffsetInModule(InOffsetInModule),
	ProgramCounter(InProgramCounter)
{
}

/** Settings for stack walking */
static bool GWantsDetailedCallstacksInNonMonolithicBuilds = true;

void FGenericPlatformStackWalk::Init()
{
	// This needs to be called once configs are initialized
	check(GConfig);

	GConfig->GetBool(TEXT("Core.System"), TEXT("DetailedCallstacksInNonMonolithicBuilds"), GWantsDetailedCallstacksInNonMonolithicBuilds, GEngineIni);
}

bool FGenericPlatformStackWalk::WantsDetailedCallstacksInNonMonolithicBuilds()
{
	return GWantsDetailedCallstacksInNonMonolithicBuilds;
}

bool FGenericPlatformStackWalk::ProgramCounterToHumanReadableString( int32 CurrentCallDepth, uint64 ProgramCounter, ANSICHAR* HumanReadableString, SIZE_T HumanReadableStringSize, FGenericCrashContext* Context )
{
	if (HumanReadableString && HumanReadableStringSize > 0)
	{
		FProgramCounterSymbolInfo SymbolInfo;
		FPlatformStackWalk::ProgramCounterToSymbolInfo( ProgramCounter, SymbolInfo );

		return FPlatformStackWalk::SymbolInfoToHumanReadableString( SymbolInfo, HumanReadableString, HumanReadableStringSize );
	}
	return false;
}

bool FGenericPlatformStackWalk::SymbolInfoToHumanReadableString( const FProgramCounterSymbolInfo& SymbolInfo, ANSICHAR* HumanReadableString, SIZE_T HumanReadableStringSize )
{
	const int32 MAX_TEMP_SPRINTF = 256;

	//
	// Callstack lines should be written in this standard format
	//
	//	0xaddress module!func [file]
	// 
	// E.g. 0x045C8D01 OrionClient.self!UEngine::PerformError() [D:\Epic\Orion\Engine\Source\Runtime\Engine\Private\UnrealEngine.cpp:6481]
	//
	// Module may be omitted, everything else should be present, or substituted with a string that conforms to the expected type
	//
	// E.g 0x00000000 UnknownFunction []
	//
	// 
	if( HumanReadableString && HumanReadableStringSize > 0 )
	{
		ANSICHAR StackLine[MAX_SPRINTF] = {0};

		// Strip module path.
		const ANSICHAR* Pos0 = FCStringAnsi::Strrchr( SymbolInfo.ModuleName, '\\' );
		const ANSICHAR* Pos1 = FCStringAnsi::Strrchr( SymbolInfo.ModuleName, '/' );
		const UPTRINT RealPos = FMath::Max( (UPTRINT)Pos0, (UPTRINT)Pos1 );
		const ANSICHAR* StrippedModuleName = RealPos > 0 ? (const ANSICHAR*)(RealPos + 1) : SymbolInfo.ModuleName;

		// Start with address
		ANSICHAR PCAddress[MAX_TEMP_SPRINTF] = { 0 };
		FCStringAnsi::Snprintf(PCAddress, MAX_TEMP_SPRINTF, "0x%016llx ", SymbolInfo.ProgramCounter);
		FCStringAnsi::StrncatTruncateDest(StackLine, MAX_SPRINTF, PCAddress);
		
		// Module if it's present
		const bool bHasValidModuleName = FCStringAnsi::Strlen(StrippedModuleName) > 0;
		if (bHasValidModuleName)
		{
			FCStringAnsi::StrncatTruncateDest(StackLine, MAX_SPRINTF, StrippedModuleName);
			FCStringAnsi::StrncatTruncateDest(StackLine, MAX_SPRINTF, "!");
		}

		// Function if it's available, unknown if it's not
		const bool bHasValidFunctionName = FCStringAnsi::Strlen( SymbolInfo.FunctionName ) > 0;
		if( bHasValidFunctionName )
		{
			FCStringAnsi::StrncatTruncateDest(StackLine, MAX_SPRINTF, SymbolInfo.FunctionName);
		}
		else
		{
			FCStringAnsi::StrncatTruncateDest(StackLine, MAX_SPRINTF, "UnknownFunction");
		}

		// file info
		const bool bHasValidFilename = FCStringAnsi::Strlen( SymbolInfo.Filename ) > 0 && SymbolInfo.LineNumber > 0;
		if( bHasValidFilename )
		{
			ANSICHAR FilenameAndLineNumber[MAX_TEMP_SPRINTF] = {0};
			FCStringAnsi::Snprintf( FilenameAndLineNumber, MAX_TEMP_SPRINTF, " [%s:%i]", SymbolInfo.Filename, SymbolInfo.LineNumber );
			FCStringAnsi::StrncatTruncateDest(StackLine, MAX_SPRINTF, FilenameAndLineNumber);
		}
		else
		{
			FCStringAnsi::StrncatTruncateDest(StackLine, MAX_SPRINTF, " []");
		}

		// Append the stack line.
		FCStringAnsi::StrncatTruncateDest(HumanReadableString, (int32)HumanReadableStringSize, StackLine);

		// Return true, if we have a valid function name.
		return bHasValidFunctionName;
	}
	return false;
}


bool FGenericPlatformStackWalk::SymbolInfoToHumanReadableStringEx( const FProgramCounterSymbolInfoEx& SymbolInfo, FString& out_HumanReadableString )
{
	// Valid callstack line 
	// ModuleName!FunctionName [Filename:LineNumber]
	// 
	// Invalid callstack line
	// ModuleName! {ProgramCounter}
	
	// Strip module path.
	int32 Pos0;
	int32 Pos1;
	SymbolInfo.ModuleName.FindLastChar(TEXT('\\'), Pos0);
	SymbolInfo.ModuleName.FindLastChar(TEXT('/'), Pos1);
	const int32 RealPos = FMath::Max( Pos0, Pos1 );
	const FString StrippedModuleName = RealPos > 0 ? SymbolInfo.ModuleName.RightChop(RealPos + 1) : SymbolInfo.ModuleName;

	out_HumanReadableString = StrippedModuleName;
	
	const bool bHasValidFunctionName = SymbolInfo.FunctionName.Len() > 0;
	if( bHasValidFunctionName )
	{	
		out_HumanReadableString += TEXT( "!" );
		out_HumanReadableString += SymbolInfo.FunctionName;
	}

	const bool bHasValidFilename = SymbolInfo.Filename.Len() > 0 && SymbolInfo.LineNumber > 0;
	if( bHasValidFilename )
	{
		out_HumanReadableString += FString::Printf( TEXT( " [%s:%i]" ), *SymbolInfo.Filename, SymbolInfo.LineNumber );
	}

	// Return true, if we have a valid function name.
	return bHasValidFunctionName;
}


uint32 FGenericPlatformStackWalk::CaptureStackBackTrace( uint64* BackTrace, uint32 MaxDepth, void* Context )
{
	return 0;
}

uint32 FGenericPlatformStackWalk::CaptureThreadStackBackTrace(uint64 ThreadId, uint64* BackTrace, uint32 MaxDepth, void* Context)
{
	return 0;
}

void FGenericPlatformStackWalk::StackWalkAndDump( ANSICHAR* HumanReadableString, SIZE_T HumanReadableStringSize, int32 IgnoreCount, void* Context )
{
	// Temporary memory holding the stack trace.
	static const int MAX_DEPTH = 100;
	uint64 StackTrace[MAX_DEPTH];
	FMemory::Memzero( StackTrace );

	// If the callstack is for the executing thread, ignore this function and the FPlatformStackWalk::CaptureStackBackTrace call below
	if(Context == nullptr)
	{
		IgnoreCount += 2;
	}

	// Capture stack backtrace.
	uint32 Depth = FPlatformStackWalk::CaptureStackBackTrace( StackTrace, MAX_DEPTH, Context );

	// Skip the first two entries as they are inside the stack walking code.
	uint32 CurrentDepth = IgnoreCount;
	while( CurrentDepth < Depth )
	{
		FPlatformStackWalk::ProgramCounterToHumanReadableString( CurrentDepth, StackTrace[CurrentDepth], HumanReadableString, HumanReadableStringSize, reinterpret_cast< FGenericCrashContext* >( Context ) );
		FCStringAnsi::StrncatTruncateDest(HumanReadableString, (int32)HumanReadableStringSize, LINE_TERMINATOR_ANSI);
		CurrentDepth++;
	}
}

void FGenericPlatformStackWalk::StackWalkAndDump( ANSICHAR* HumanReadableString, SIZE_T HumanReadableStringSize, void* ProgramCounter, void* Context )
{
	uint64 StackTrace[100];
	int32 Depth = FPlatformStackWalk::CaptureStackBackTrace(StackTrace, UE_ARRAY_COUNT(StackTrace), Context);

	int32 CurrentDepth = 0;
	if (ProgramCounter != nullptr)
	{
		for (int32 i = 0; i < Depth; ++i)
		{
			if (StackTrace[i] != uint64(ProgramCounter))
			{
				continue;
			}

			CurrentDepth = i;
			break;
		}
	}

	for (; CurrentDepth < Depth; CurrentDepth++)
	{
		FPlatformStackWalk::ProgramCounterToHumanReadableString( CurrentDepth, StackTrace[CurrentDepth], HumanReadableString, HumanReadableStringSize, reinterpret_cast< FGenericCrashContext* >( Context ) );
		FCStringAnsi::StrncatTruncateDest(HumanReadableString, (int32)HumanReadableStringSize, LINE_TERMINATOR_ANSI);
	}
}

void FGenericPlatformStackWalk::StackWalkAndDumpEx(ANSICHAR* HumanReadableString, SIZE_T HumanReadableStringSize, int32 IgnoreCount, uint32 Flags, void* Context)
{
	// generic implementation ignores extra flags
	return FPlatformStackWalk::StackWalkAndDump(HumanReadableString, HumanReadableStringSize, IgnoreCount, Context);
}

void FGenericPlatformStackWalk::StackWalkAndDumpEx(ANSICHAR* HumanReadableString, SIZE_T HumanReadableStringSize, void* ProgramCounter, uint32 Flags, void* Context)
{
	return FPlatformStackWalk::StackWalkAndDump(HumanReadableString, HumanReadableStringSize, ProgramCounter, Context);
}

TArray<FProgramCounterSymbolInfo> FGenericPlatformStackWalk::GetStack(int32 IgnoreCount, int32 MaxDepth, void* Context)
{
	TArray<FProgramCounterSymbolInfo> Stack;

	// Temporary memory holding the stack trace.
	static const int MAX_DEPTH = 100;
	uint64 StackTrace[MAX_DEPTH];
	FMemory::Memzero(StackTrace);

	// If the callstack is for the executing thread, ignore this function and the FPlatformStackWalk::CaptureStackBackTrace call below
	if(Context == nullptr)
	{
		IgnoreCount += 2;
	}

	MaxDepth = FMath::Min(MAX_DEPTH, IgnoreCount + MaxDepth);

	// Capture stack backtrace.
	uint32 Depth = FPlatformStackWalk::CaptureStackBackTrace(StackTrace, MaxDepth, Context);

	// Skip the first two entries as they are inside the stack walking code.
	uint32 CurrentDepth = IgnoreCount;
	while ( CurrentDepth < Depth )
	{
		int32 NewIndex = Stack.AddDefaulted();
		FPlatformStackWalk::ProgramCounterToSymbolInfo(StackTrace[CurrentDepth], Stack[NewIndex]);
		CurrentDepth++;
	}

	return Stack;
}

TMap<FName, FString> FGenericPlatformStackWalk::GetSymbolMetaData()
{
	return TMap<FName, FString>();
}

void FGenericPlatformStackWalk::CopyVirtualPathToLocal(char* Dest, int32 DestCapacity, const char* Source)
{
	const char* VfsPathsIt = BuildSettings::GetVfsPaths();
	while (true)
	{
		const char* VirtualPath = VfsPathsIt;
		const char* VirtualPathEnd = FCStringAnsi::Strchr(VirtualPath, ';');
		if (!VirtualPathEnd)
		{
			break;
		}

		const char* LocalPath = VirtualPathEnd + 1;
		const char* LocalPathEnd = FCStringAnsi::Strchr(LocalPath, ';');

		int32 VirtualPathLen = int32(VirtualPathEnd - VirtualPath);

		if (FCStringAnsi::Strncmp(Source, VirtualPath, VirtualPathLen) == 0)
		{
			int32 LocalPathLen = int32(LocalPathEnd - LocalPath);

			FCStringAnsi::Strncpy(Dest, LocalPath, FMath::Min(DestCapacity, LocalPathLen+1));
			FCStringAnsi::StrncatTruncateDest(Dest, DestCapacity, Source + VirtualPathLen);
			return;
		}

		VfsPathsIt = LocalPathEnd + 1;
	}

	FCStringAnsi::Strncpy(Dest, Source, DestCapacity);
}
