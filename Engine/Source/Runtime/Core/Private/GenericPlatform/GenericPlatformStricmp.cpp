// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenericPlatform/GenericPlatformStricmp.h"
#include "Misc/Char.h"

static constexpr uint8 LowerAscii[128] = {
	0x00, 0x01, 0x02, 0x03, 0x04, 0x05, 0x06, 0x07, 0x08, 0x09, 0x0A, 0x0B, 0x0C, 0x0D, 0x0E, 0x0F,
	0x10, 0x11, 0x12, 0x13, 0x14, 0x15, 0x16, 0x17, 0x18, 0x19, 0x1A, 0x1B, 0x1C, 0x1D, 0x1E, 0x1F,
	0x20, 0x21, 0x22, 0x23, 0x24, 0x25, 0x26, 0x27, 0x28, 0x29, 0x2A, 0x2B, 0x2C, 0x2D, 0x2E, 0x2F,
	0x30, 0x31, 0x32, 0x33, 0x34, 0x35, 0x36, 0x37, 0x38, 0x39, 0x3A, 0x3B, 0x3C, 0x3D, 0x3E, 0x3F,
	0x40, 'a',  'b',  'c',  'd',  'e',  'f',  'g',  'h',  'i',  'j',  'k',  'l',  'm',  'n',  'o',
	'p',  'q',  'r',  's',  't',  'u',  'v',  'w',  'x',  'y',  'z',  0x5B, 0x5C, 0x5D, 0x5E, 0x5F,
	0x60, 0x61, 0x62, 0x63, 0x64, 0x65, 0x66, 0x67, 0x68, 0x69, 0x6A, 0x6B, 0x6C, 0x6D, 0x6E, 0x6F,
	0x70, 0x71, 0x72, 0x73, 0x74, 0x75, 0x76, 0x77, 0x78, 0x79, 0x7A, 0x7B, 0x7C, 0x7D, 0x7E, 0x7F
};

FORCEINLINE bool BothAscii(uint32 C1, uint32 C2)
{
	return ((C1 | C2) & 0xffffff80) == 0;
}

template<typename CharType1, typename CharType2>
int32 StricmpImpl(const CharType1* String1, const CharType2* String2)
{
	while (true)
	{
		CharType1 C1 = *String1++;
		CharType2 C2 = *String2++;

		uint32 U1 = TChar<CharType1>::ToUnsigned(C1);
		uint32 U2 = TChar<CharType2>::ToUnsigned(C2);

		// Quickly move on if characters are identical but
		// return equals if we found two null terminators
		if (U1 == U2)
		{
			if (U1)
			{
				continue;
			}

			return 0;
		}
		else if (BothAscii(U1, U2))
		{
			if (int32 Diff = LowerAscii[U1] - LowerAscii[U2])
			{
				return Diff;
			}
		}
		else
		{
			return U1 - U2;
		}
	}
}

template<typename CharType1, typename CharType2>
int32 StrnicmpImpl(const CharType1* String1, const CharType2* String2, SIZE_T Count)
{
	for (; Count > 0; --Count)
	{
		CharType1 C1 = *String1++;
		CharType2 C2 = *String2++;

		// Quickly move on if characters are identical but
		// return equals if we found two null terminators
		if (C1 == C2)
		{
			if (C1)
			{
				continue;
			}

			return 0;
		}
		else if (BothAscii(C1, C2))
		{
			if (int32 Diff = LowerAscii[TChar<CharType1>::ToUnsigned(C1)] - LowerAscii[TChar<CharType2>::ToUnsigned(C2)])
			{
				return Diff;
			}
		}
		else
		{
			return TChar<CharType1>::ToUnsigned(C1) - TChar<CharType2>::ToUnsigned(C2);
		}
	}

	return 0;
}

int32 FGenericPlatformStricmp::Stricmp(const ANSICHAR* Str1, const ANSICHAR* Str2) { return StricmpImpl(Str1, Str2); }
int32 FGenericPlatformStricmp::Stricmp(const WIDECHAR* Str1, const WIDECHAR* Str2) { return StricmpImpl(Str1, Str2); }
int32 FGenericPlatformStricmp::Stricmp(const UTF8CHAR* Str1, const UTF8CHAR* Str2) { return StricmpImpl(Str1, Str2); }
int32 FGenericPlatformStricmp::Stricmp(const UTF16CHAR* Str1, const UTF16CHAR* Str2) { return StricmpImpl(Str1, Str2); }
int32 FGenericPlatformStricmp::Stricmp(const UTF32CHAR* Str1, const UTF32CHAR* Str2) { return StricmpImpl(Str1, Str2); }
int32 FGenericPlatformStricmp::Stricmp(const ANSICHAR* Str1, const WIDECHAR* Str2) { return StricmpImpl(Str1, Str2); }
int32 FGenericPlatformStricmp::Stricmp(const ANSICHAR* Str1, const UTF8CHAR* Str2) { return StricmpImpl(Str1, Str2); }
int32 FGenericPlatformStricmp::Stricmp(const ANSICHAR* Str1, const UTF16CHAR* Str2) { return StricmpImpl(Str1, Str2); }
int32 FGenericPlatformStricmp::Stricmp(const ANSICHAR* Str1, const UTF32CHAR* Str2) { return StricmpImpl(Str1, Str2); }
int32 FGenericPlatformStricmp::Stricmp(const WIDECHAR* Str1, const ANSICHAR* Str2) { return StricmpImpl(Str1, Str2); }
int32 FGenericPlatformStricmp::Stricmp(const UTF8CHAR* Str1, const ANSICHAR* Str2) { return StricmpImpl(Str1, Str2); }
int32 FGenericPlatformStricmp::Stricmp(const UTF16CHAR* Str1, const ANSICHAR* Str2) { return StricmpImpl(Str1, Str2); }
int32 FGenericPlatformStricmp::Stricmp(const UTF32CHAR* Str1, const ANSICHAR* Str2) { return StricmpImpl(Str1, Str2); }

int32 FGenericPlatformStricmp::Strnicmp(const ANSICHAR*  Str1, const ANSICHAR*  Str2, SIZE_T Count) { return StrnicmpImpl(Str1, Str2, Count); }
int32 FGenericPlatformStricmp::Strnicmp(const WIDECHAR*  Str1, const WIDECHAR*  Str2, SIZE_T Count) { return StrnicmpImpl(Str1, Str2, Count); }
int32 FGenericPlatformStricmp::Strnicmp(const UTF8CHAR*  Str1, const UTF8CHAR*  Str2, SIZE_T Count) { return StrnicmpImpl(Str1, Str2, Count); }
int32 FGenericPlatformStricmp::Strnicmp(const UTF16CHAR* Str1, const UTF16CHAR* Str2, SIZE_T Count) { return StrnicmpImpl(Str1, Str2, Count); }
int32 FGenericPlatformStricmp::Strnicmp(const UTF32CHAR* Str1, const UTF32CHAR* Str2, SIZE_T Count) { return StrnicmpImpl(Str1, Str2, Count); }
int32 FGenericPlatformStricmp::Strnicmp(const ANSICHAR*  Str1, const WIDECHAR*  Str2, SIZE_T Count) { return StrnicmpImpl(Str1, Str2, Count); }
int32 FGenericPlatformStricmp::Strnicmp(const ANSICHAR*  Str1, const UTF8CHAR*  Str2, SIZE_T Count) { return StrnicmpImpl(Str1, Str2, Count); }
int32 FGenericPlatformStricmp::Strnicmp(const ANSICHAR*  Str1, const UTF16CHAR* Str2, SIZE_T Count) { return StrnicmpImpl(Str1, Str2, Count); }
int32 FGenericPlatformStricmp::Strnicmp(const ANSICHAR*  Str1, const UTF32CHAR* Str2, SIZE_T Count) { return StrnicmpImpl(Str1, Str2, Count); }
int32 FGenericPlatformStricmp::Strnicmp(const WIDECHAR*  Str1, const ANSICHAR*  Str2, SIZE_T Count) { return StrnicmpImpl(Str1, Str2, Count); }
int32 FGenericPlatformStricmp::Strnicmp(const UTF8CHAR*  Str1, const ANSICHAR*  Str2, SIZE_T Count) { return StrnicmpImpl(Str1, Str2, Count); }
int32 FGenericPlatformStricmp::Strnicmp(const UTF16CHAR* Str1, const ANSICHAR*  Str2, SIZE_T Count) { return StrnicmpImpl(Str1, Str2, Count); }
int32 FGenericPlatformStricmp::Strnicmp(const UTF32CHAR* Str1, const ANSICHAR*  Str2, SIZE_T Count) { return StrnicmpImpl(Str1, Str2, Count); }

//////////////////////////////////////////////////////////////////////////

#if WITH_DEV_AUTOMATION_TESTS 

#include "Misc/AutomationTest.h"

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FGenericPlatformStricmpTest, "System.Core.GenericPlatform.Stricmp", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)

// Simpler reference implementation based on ToLower() instead of a lookup table.
// Used to verify correctness on non-Windows platforms
template <typename CharType>
int32 StricmpExpected(const CharType* Str1, const CharType* Str2)
{
	for (; *Str1 || *Str2; ++Str1, ++Str2)
	{
		CharType Char1 = TChar<CharType>::ToLower(*Str1);
		CharType Char2 = TChar<CharType>::ToLower(*Str2);

		if (Char1 != Char2)
		{
			return TChar<CharType>::ToUnsigned(Char1) - TChar<CharType>::ToUnsigned(Char2);
		}
	}

	return 0;
}

template<typename CharType>
void TestStricmp(const CharType* Str1, const CharType* Str2, FAutomationTestBase& Test)
{
	Test.TestEqual("Stricmp()", FMath::Sign(StricmpImpl(Str1, Str2)), FMath::Sign(StricmpExpected(Str1, Str2)));
}

template<typename CharType>
void RunStricmpTests(FAutomationTestBase& Test)
{
	// Test a range of single character strings starting with a few large ones 
	const CharType Empty[1] = { '\0' };
	for (int32 Char = -2; Char < 256; ++Char)
	{
		const CharType Current[2] = { (CharType)(Char), '\0' };
		const CharType Next[2] = { (CharType)(Char + 1), '\0' };
		const CharType CurrentPlusCasingDistance[2] = { (CharType)(Char + ('a' - 'A')), '\0' };

		TestStricmp(Current, Current, Test);
		TestStricmp(Current, Empty, Test);
		TestStricmp(Current, Next, Test);
		TestStricmp(Next, Current, Test);
		TestStricmp(Current, CurrentPlusCasingDistance, Test);
	}
	
	// Test various ASCII casings
	const CharType HelloLower[] = { 'h', 'e', 'l', 'l', 'o', '\0' };
	const CharType HelloUpper[] = { 'H', 'E', 'L', 'L', 'O', '\0' };
	const CharType HelloMixed1[] = { 'H', 'e', 'L', 'L', 'o', '\0' };
	const CharType HelloMixed2[] = { 'h', 'E', 'l', 'l', 'O', '\0' };
	const CharType Hell0[] = { 'h', 'e', 'l', 'l', '0', '\0' };

	TestStricmp(HelloLower, HelloLower, Test);
	TestStricmp(HelloLower, HelloUpper, Test);
	TestStricmp(HelloLower, HelloMixed1, Test);
	TestStricmp(HelloLower, HelloMixed2, Test);
	TestStricmp(HelloLower, Hell0, Test);
}

bool FGenericPlatformStricmpTest::RunTest(const FString& Parameters)
{
	RunStricmpTests<ANSICHAR>(*this);
	RunStricmpTests<WIDECHAR>(*this);

	return true;
}

#endif //WITH_DEV_AUTOMATION_TESTS