// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenericPlatform/GenericPlatformString.h"
#include "GenericPlatform/GenericWidePlatformString.h"
#include "GenericPlatform/StandardPlatformString.h"
#include "HAL/UnrealMemory.h"
#include "Misc/Char.h"
#include "Containers/StringConv.h"
#include "Containers/UnrealString.h"
#include "Logging/LogMacros.h"


DEFINE_LOG_CATEGORY_STATIC(LogGenericPlatformString, Log, All);

template <> const TCHAR* FGenericPlatformString::GetEncodingTypeName<ANSICHAR>() { return TEXT("ANSICHAR"); }
template <> const TCHAR* FGenericPlatformString::GetEncodingTypeName<WIDECHAR>() { return TEXT("WIDECHAR"); }
template <> const TCHAR* FGenericPlatformString::GetEncodingTypeName<UCS2CHAR>() { return TEXT("UCS2CHAR"); }
template <> const TCHAR* FGenericPlatformString::GetEncodingTypeName<UTF8CHAR>() { return TEXT("UTF8CHAR"); }
#if PLATFORM_TCHAR_IS_CHAR16
template <> const TCHAR* FGenericPlatformString::GetEncodingTypeName<wchar_t>() { return TEXT("WCHAR_T"); }
#endif

void* FGenericPlatformString::Memcpy(void* Dest, const void* Src, SIZE_T Count)
{
	return FMemory::Memcpy(Dest, Src, Count);
}

#if DO_CHECK
void FGenericPlatformString::ReportZeroStrncpySize()
{
	checkf(false, TEXT("MaxLen cannot be zero as Strncpy expects to append a zero terminator"));
}
#endif

namespace
{
	// Output iterator which will not overflow the destination buffer but will keep track of how many characters have been written.
	// It will also null terminate the output when destructed.
	template <typename CharType>
	struct TSafeDestIterator
	{
		// Non-copyable
		TSafeDestIterator(const TSafeDestIterator&) = delete;
		TSafeDestIterator& operator=(const TSafeDestIterator&) = delete;

		explicit TSafeDestIterator(CharType* InPtr, SIZE_T InSize)
			: NumCharsWritten(0)
			, Ptr(InSize ? InPtr : nullptr)
			, EndMinusOne(InSize ? InPtr + InSize - 1 : nullptr)
		{
		}

		FORCENOINLINE ~TSafeDestIterator()
		{
			// Auto-terminate buffer
			if (Ptr)
			{
				*Ptr = 0;
			}
		}

		// Writes Count instances of Ch to the destination buffer
		FORCENOINLINE TSafeDestIterator& Write(CharType Ch, int32 Count = 1)
		{
			NumCharsWritten += Count;

			// Ensure we're not in the error state
			if (EndMinusOne)
			{
				int32 NumToWrite = FPlatformMath::Min(Count, (int32)(EndMinusOne - Ptr));
				for (int32 Val = NumToWrite; Val; --Val)
				{
					*Ptr++ = Ch;
				}

				if (NumToWrite != Count)
				{
					EndMinusOne = nullptr;
				}
			}

			return *this;
		}

		// Writes Count characters from Src to the destination buffer
		template <typename OtherCharType>
		FORCENOINLINE TSafeDestIterator& Write(const OtherCharType* Src, int32 Count)
		{
			using DstType = std::conditional_t<std::is_same_v<ANSICHAR, CharType>, UTF8CHAR, CharType>;
			using SrcType = std::conditional_t<std::is_same_v<ANSICHAR, OtherCharType>, UTF8CHAR, OtherCharType>;
			if constexpr (!std::is_same_v<DstType, SrcType>)
			{
				int32 ConvertedLength = FPlatformString::ConvertedLength<DstType, SrcType>((const SrcType*)Src, Count);
				NumCharsWritten += ConvertedLength;

				// Ensure we're not in the error state
				if (EndMinusOne)
				{
					// Copy as much of the source as will fit in the destination.
					// Convert technically has undefined behavior when the destination size is
					// less than the required size, but it does write as many full code points as
					// it can. This may leave 1-3 unwritten code units before the null terminator.
					int32 NumToWrite = FPlatformMath::Min(ConvertedLength, (int32)(EndMinusOne - Ptr));
					CharType* Result = (CharType*)FPlatformString::Convert((DstType*)Ptr, NumToWrite, (const SrcType*)Src, Count);
					if (NumToWrite == ConvertedLength)
					{
						check(Result);
						Ptr = Result;
					}
					else
					{
						Ptr = Ptr + NumToWrite;
						EndMinusOne = nullptr;
					}
				}
			}
			else
			{
				NumCharsWritten += Count;

				// Ensure we're not in the error state
				if (EndMinusOne)
				{
					int32 NumToWrite = FPlatformMath::Min(Count, (int32)(EndMinusOne - Ptr));

					// Copy as much of the source as will fit in the destination
					for (int32 Val = NumToWrite; Val; --Val)
					{
						*Ptr++ = *Src++;
					}

					if (NumToWrite != Count)
					{
						EndMinusOne = nullptr;
					}
				}
			}

			return *this;
		}

		int32 GetNumCharsWritten() const
		{
			return NumCharsWritten;
		}

	private:
		int32     NumCharsWritten;
		CharType* Ptr;
		CharType* EndMinusOne; // when null, this means the iterator has already moved past the writable area of the buffer
	};

	static inline bool CharIsIntegerFormatSpecifier(TCHAR Char)
	{
		return (Char == 'i') | (Char == 'd') | (Char == 'u') | (Char == 'X') | (Char == 'x');
	}

	template <typename FormatCharType, typename ArgCharType>
	void WriteStringArg(TSafeDestIterator<FormatCharType>& DestIter, const ArgCharType* Val, int FieldLen, int PrecisionLen)
	{
		int RetCnt = PrecisionLen < 0 ? FPlatformString::Strlen(Val) : FPlatformString::Strnlen(Val, PrecisionLen);

		int Spaces = FPlatformMath::Max(FPlatformMath::Abs(FieldLen) - RetCnt, 0);
		if (Spaces > 0 && FieldLen > 0)
		{
			DestIter.Write(TEXT(' '), Spaces);
		}
		DestIter.Write(Val, RetCnt);
		if (Spaces > 0 && FieldLen < 0)
		{
			DestIter.Write(TEXT(' '), Spaces);
		}
	}

	template <typename ArgCharType, typename FormatCharType, typename VaListType>
	void ProcessStringArg(TSafeDestIterator<FormatCharType>& DestIter, const FormatCharType*& Src, int FieldLen, int PrecisionLen, VaListType& ArgPtr)
	{
		// ArgPtr is taken as a templated parameter, despite being a va_list, because on some
		// architectures it could be an array type, and when an array is used as a function parameter,
		// the named value is treated as a pointer and won't bind to an array reference.

		Src++;
		static const ArgCharType* Null = LITERAL(ArgCharType, "(null)");
		const ArgCharType* Val = va_arg(ArgPtr, ArgCharType*);
		if (Val == nullptr)
		{
			Val = Null;
		}

		WriteStringArg(DestIter, Val, FieldLen, PrecisionLen);
	}

	static void LogUnrecognizedFormatCharacters(const char* MsgFormat, WIDECHAR Char, const WIDECHAR* InputFormat)													{ printf(MsgFormat, (wint_t)Char, TCHAR_TO_ANSI(InputFormat)); }
	static void LogUnrecognizedFormatCharacters(const char* MsgFormat, WIDECHAR Char1, WIDECHAR Char2, const WIDECHAR* InputFormat)									{ printf(MsgFormat, (wint_t)Char1, (wint_t)Char2, TCHAR_TO_ANSI(InputFormat)); }
	static void LogUnrecognizedFormatCharacters(const char* MsgFormat, WIDECHAR Char1, WIDECHAR Char2, WIDECHAR Char3, const WIDECHAR* InputFormat)					{ printf(MsgFormat, (wint_t)Char1, (wint_t)Char2, (wint_t)Char3, TCHAR_TO_ANSI(InputFormat)); }
	static void LogUnrecognizedFormatCharacters(const char* MsgFormat, WIDECHAR Char1, WIDECHAR Char2, WIDECHAR Char3, WIDECHAR Char4, const WIDECHAR* InputFormat)	{ printf(MsgFormat, (wint_t)Char1, (wint_t)Char2, (wint_t)Char3, (wint_t)Char4, TCHAR_TO_ANSI(InputFormat)); }

	static void LogUnrecognizedFormatCharacters(const char* MsgFormat, ANSICHAR Char, const ANSICHAR* InputFormat)													{ printf(MsgFormat, Char, InputFormat); }
	static void LogUnrecognizedFormatCharacters(const char* MsgFormat, ANSICHAR Char1, ANSICHAR Char2, const ANSICHAR* InputFormat)									{ printf(MsgFormat, Char1, Char2, InputFormat); }
	static void LogUnrecognizedFormatCharacters(const char* MsgFormat, ANSICHAR Char1, ANSICHAR Char2, ANSICHAR Char3, const ANSICHAR* InputFormat)					{ printf(MsgFormat, Char1, Char2, Char3, InputFormat); }
	static void LogUnrecognizedFormatCharacters(const char* MsgFormat, ANSICHAR Char1, ANSICHAR Char2, ANSICHAR Char3, ANSICHAR Char4, const ANSICHAR* InputFormat) { printf(MsgFormat, Char1, Char2, Char3, Char4, InputFormat); }

	template <typename CharType, typename VaListType>
	static int32 GetVarArgsImpl(CharType* Dest, SIZE_T DestSize, const CharType*& Fmt, VaListType& ArgPtr)
	{
		if (Fmt == nullptr)
		{
			if ((DestSize > 0) && (Dest != nullptr))
			{
				*Dest = 0;
			}
			return 0;
		}

		const CharType* Src = Fmt;

		TSafeDestIterator<CharType> DestIter(Dest, DestSize);
		while (*Src)
		{
			if (*Src != '%')
			{
				DestIter.Write(*Src++);
				continue;
			}

			const CharType* Percent = Src;
			int FieldLen = 0;
			bool bVariablePrecision = false;
			int PrecisionLen = -1;

			Src++; // skip the '%' char...

			// Skip modifier flags that don't need additional processing;
			// they still get passed to snprintf() below based on the conversion.
			while (*Src == '+' || *Src == '#' || *Src == TEXT(' ') || *Src == TEXT('0'))
			{
				Src++;
			}

			// check for field width requests...
			if ((*Src == '-') || ((*Src >= '0') && (*Src <= '9')))
			{
				const CharType* Cur = Src + 1;
				while ((*Cur >= '0') && (*Cur <= '9'))
				{
					Cur++;
				}

				FieldLen = FPlatformString::Atoi(Src);
				Src = Cur;
			}

			// check for dynamic field requests
			if (*Src == '*')
			{
				FieldLen = va_arg(ArgPtr, int32);
				Src++;
			}

			if (*Src == '.')
			{
				const CharType* Cur = Src + 1;
				if (*Cur == '*')
				{
					bVariablePrecision = true;
					PrecisionLen = va_arg(ArgPtr, int32);
					Cur++;
				}
				else
				{
					while ((*Cur >= '0') && (*Cur <= '9'))
					{
						Cur++;
					}

					PrecisionLen = FPlatformString::Atoi(Src + 1);
				}
				Src = Cur;
			}

			bool bIsNarrowString = std::is_same_v<CharType, ANSICHAR>;
			// Check for 'ls' field, change to 's' and remember
			if ((Src[0] == 'l' && Src[1] == 's'))
			{
				bIsNarrowString = false;
				Src++;
			}

			// Check for 'hs' field, change to 's' and remember
			if ((Src[0] == 'h' && Src[1] == 's'))
			{
				bIsNarrowString = true;
				Src++;
			}

			switch (*Src)
			{
			case '%':
			{
				Src++;
				DestIter.Write('%');
				break;
			}

			case 'c':
			{
				CharType Val = (CharType)va_arg(ArgPtr, int);
				Src++;
				if (FieldLen > 1)
				{
					DestIter.Write(CharType(' '), FieldLen - 1);
				}
				DestIter.Write(Val);
				if (FieldLen < -1)
				{
					DestIter.Write(CharType(' '), FPlatformMath::Abs(FieldLen + 1));
				}
				break;
			}

			case 'd':
			case 'i':
			case 'X':
			case 'x':
			case 'u':
			{
				Src++;
				int Val = va_arg(ArgPtr, int);
				ANSICHAR AnsiNum[64];
				ANSICHAR FmtBuf[30];

				// limit width to the buffer size
				FieldLen = FMath::Min(static_cast<int>(sizeof(AnsiNum)) - 1, FieldLen);

				// Yes, this is lame.
				int CpyIdx = 0;
				while (Percent < Src && CpyIdx < UE_ARRAY_COUNT(FmtBuf))
				{
					if (LIKELY(*Percent != '*'))
					{
						FmtBuf[CpyIdx] = (ANSICHAR)*Percent;
						++CpyIdx;
					}
					else
					{
						snprintf(&FmtBuf[CpyIdx], sizeof(FmtBuf) - CpyIdx, "%d", FieldLen);
						while (CpyIdx < UE_ARRAY_COUNT(FmtBuf) && FmtBuf[CpyIdx] != 0)
						{
							++CpyIdx;
						}
					}
					++Percent;
				}
				FmtBuf[CpyIdx] = 0;

				int RetCnt = snprintf(AnsiNum, sizeof(AnsiNum), FmtBuf, Val);
				if (RetCnt < 0)
				{
					return -1;
				}
				DestIter.Write(AnsiNum, RetCnt);
				break;
			}

			case 'z':
			case 'Z':
			{
				if (Src[1] == 0)
				{
					LogUnrecognizedFormatCharacters("Unknown percent [%lc] in FGenericWidePlatformString::GetVarArgs() [%s]\n.", Src[0], Fmt);
					Src++;  // skip it, I guess.
					break;
				}

				Src += 2;

				size_t Val = va_arg(ArgPtr, size_t);

				ANSICHAR AnsiNum[64];
				ANSICHAR FmtBuf[30];

				// limit width to the buffer size
				FieldLen = FMath::Min(static_cast<int>(sizeof(AnsiNum)) - 1, FieldLen);

				// Yes, this is lame.
				int CpyIdx = 0;
				while (Percent < Src && CpyIdx < UE_ARRAY_COUNT(FmtBuf))
				{
					if (LIKELY(*Percent != '*'))
					{
						FmtBuf[CpyIdx] = (ANSICHAR)*Percent;
						++CpyIdx;
					}
					else
					{
						snprintf(&FmtBuf[CpyIdx], sizeof(FmtBuf) - CpyIdx, "%d", FieldLen);
						while (CpyIdx < UE_ARRAY_COUNT(FmtBuf) && FmtBuf[CpyIdx] != 0)
						{
							++CpyIdx;
						}
					}
					++Percent;
				}
				FmtBuf[CpyIdx] = 0;

				int RetCnt = snprintf(AnsiNum, sizeof(AnsiNum), FmtBuf, Val);
				if (RetCnt < 0)
				{
					return -1;
				}
				DestIter.Write(AnsiNum, RetCnt);
				break;
			}

			case 'p':
			{
				Src++;
				void* Val = va_arg(ArgPtr, void*);
				ANSICHAR AnsiNum[30];
				ANSICHAR FmtBuf[30];

				// Yes, this is lame.
				int CpyIdx = 0;
				while (Percent < Src && CpyIdx < UE_ARRAY_COUNT(FmtBuf))
				{
					FmtBuf[CpyIdx] = (ANSICHAR)*Percent;
					Percent++;
					CpyIdx++;
				}
				FmtBuf[CpyIdx] = 0;

				int RetCnt = snprintf(AnsiNum, sizeof(AnsiNum), FmtBuf, Val);
				if (RetCnt < 0)
				{
					return -1;
				}
				DestIter.Write(AnsiNum, RetCnt);
				break;
			}

			case 'l':
			case 'I':
			case 'h':
			{
				int RemainingSize = FPlatformString::Strlen(Src);

				if (RemainingSize < 2)
				{
					LogUnrecognizedFormatCharacters("Unknown percent [%lc] in FGenericPlatformString::GetVarArgs() [%s]\n.", Src[0], Fmt);
					Src++;  // skip it, I guess.
					break;
				}

				// treat %ld as %d. Also shorts for %h will be promoted to ints. This path also handles %li, %lu, %lx and %lX.
				if ((Src[0] == 'l' || Src[0] == 'h') && CharIsIntegerFormatSpecifier(Src[1]))
				{
					Src += 2;
					long int Val = va_arg(ArgPtr, long int);
					ANSICHAR AnsiNum[30];
					ANSICHAR FmtBuf[30];

					// Yes, this is lame.
					int CpyIdx = 0;
					while (Percent < Src && CpyIdx < UE_ARRAY_COUNT(FmtBuf))
					{
						FmtBuf[CpyIdx] = (ANSICHAR)*Percent;
						Percent++;
						CpyIdx++;
					}
					FmtBuf[CpyIdx] = 0;

					int RetCnt = snprintf(AnsiNum, sizeof(AnsiNum), FmtBuf, Val);
					if (RetCnt < 0)
					{
						return -1;
					}
					DestIter.Write(AnsiNum, RetCnt);
					break;
				}
				// Treat %lf as a %f
				else if (Src[0] == 'l' && Src[1] == 'f')
				{
					Src += 2;
					double Val = va_arg(ArgPtr, double);
					ANSICHAR AnsiNum[340]; // make sure buffer is large enough to handle the longest double values, such as the largest number which is 316 characters including decimal point and 6 decimals
					ANSICHAR FmtBuf[30];

					// Yes, this is lame.
					int CpyIdx = 0;
					while (Percent < Src && CpyIdx < UE_ARRAY_COUNT(FmtBuf))
					{
						FmtBuf[CpyIdx] = (ANSICHAR)*Percent;
						Percent++;
						CpyIdx++;
					}
					FmtBuf[CpyIdx] = 0;

					int RetCnt = snprintf(AnsiNum, sizeof(AnsiNum), FmtBuf, Val);
					if (RetCnt < 0)
					{
						return -1;
					}
					DestIter.Write(AnsiNum, RetCnt);
					break;
				}

				// At this point we expect RemainingSize >= 3, but we know it's at least 2.
				if (RemainingSize < 3)
				{
					LogUnrecognizedFormatCharacters("Unknown percent [%lc%lc] in FGenericPlatformString::GetVarArgs() [%s]\n.", Src[0], Src[1], Fmt);
					Src++;  // skip it, I guess.
					break;
				}

				// Deal with half-half formats - %hhd, %hhi, %hhu, %hhx and %hhX.
				if (Src[0] == 'h')
				{
					if (Src[1] != 'h' || !CharIsIntegerFormatSpecifier(Src[2]))
					{
						LogUnrecognizedFormatCharacters("Unknown percent [%lc%lc%lc] in FGenericPlatformString::GetVarArgs() [%s]\n.", Src[0], Src[1], Src[2], Fmt);
						Src++;  // skip it, I guess.
						break;
					}
					Src += 3;
					int Val = va_arg(ArgPtr, int);
					ANSICHAR AnsiNum[8];
					ANSICHAR FmtBuf[30];

					// Yes, this is lame.
					int CpyIdx = 0;
					while (Percent < Src && CpyIdx < UE_ARRAY_COUNT(FmtBuf))
					{
						FmtBuf[CpyIdx] = (ANSICHAR)*Percent;
						Percent++;
						CpyIdx++;
					}
					FmtBuf[CpyIdx] = 0;

					int RetCnt = snprintf(AnsiNum, sizeof(AnsiNum), FmtBuf, Val);
					if (RetCnt < 0)
					{
						return -1;
					}
					DestIter.Write(AnsiNum, RetCnt);
					break;
				}
				else if (Src[0] == 'l')
				{
					if (Src[1] != 'l' || !CharIsIntegerFormatSpecifier(Src[2]))
					{
						LogUnrecognizedFormatCharacters("Unknown percent [%lc%lc%lc] in FGenericPlatformString::GetVarArgs() [%s]\n.", Src[0], Src[1], Src[2], Fmt);
						Src++;  // skip it, I guess.
						break;
					}
				}
				else if (Src[0] == 'I')
				{
					if (RemainingSize < 4)
					{
						LogUnrecognizedFormatCharacters("Unknown percent [%lc%lc%lc] in FGenericPlatformString::GetVarArgs() [%s]\n.", Src[0], Src[1], Src[2], Fmt);
						Src++;  // skip it, I guess.
						break;
					}
					else if (Src[1] != '6' || Src[2] != '4' || !CharIsIntegerFormatSpecifier(Src[3]))
					{
						LogUnrecognizedFormatCharacters("Unknown percent [%lc%lc%lc%lc] in FGenericPlatformString::GetVarArgs() [%s]\n.", Src[0], Src[1], Src[2], Src[3], Fmt);
						Src++;  // skip it, I guess.
						break;
					}
				}
				else
				{
					LogUnrecognizedFormatCharacters("Unknown percent [%lc%lc%lc] in FGenericPlatformString::GetVarArgs() [%s]\n.", Src[0], Src[1], Src[2], Fmt);
					Src++;  // skip it, I guess.
					break;
				}

				// Yes, this is lame.
				unsigned long long Val = va_arg(ArgPtr, unsigned long long);
				ANSICHAR AnsiNum[60];
				ANSICHAR FmtBuf[30];

				int CpyIdx;
				if (Src[0] == 'I')
				{
					// Convert I64 syntax to ll 
					Src += 4;
					Percent += 4;
					CpyIdx = 3;
					FmtBuf[0] = '%';
					FmtBuf[1] = 'l';
					FmtBuf[2] = 'l';
				}
				else
				{
					Src += 3;
					CpyIdx = 0;
				}

				while (Percent < Src && CpyIdx < UE_ARRAY_COUNT(FmtBuf))
				{
					FmtBuf[CpyIdx] = (ANSICHAR)*Percent;
					Percent++;
					CpyIdx++;
				}
				FmtBuf[CpyIdx] = 0;

				int RetCnt = snprintf(AnsiNum, sizeof(AnsiNum), FmtBuf, Val);
				if (RetCnt < 0)
				{
					return -1;
				}
				DestIter.Write(AnsiNum, RetCnt);
				break;
			}

			case 'f':
			case 'e':
			case 'g':
			{
				Src++;
				double Val = va_arg(ArgPtr, double);
				ANSICHAR AnsiNum[340]; // make sure buffer is large enough to handle the longest double values, such as the largest number which is 316 characters including decimal point and 6 decimals
				ANSICHAR FmtBuf[30];

				// Yes, this is lame.
				int CpyIdx = 0;
				while (Percent < Src && CpyIdx < UE_ARRAY_COUNT(FmtBuf))
				{
					FmtBuf[CpyIdx] = (ANSICHAR)*Percent;
					Percent++;
					CpyIdx++;
				}
				FmtBuf[CpyIdx] = 0;

				int RetCnt;
				if (bVariablePrecision)
				{
					RetCnt = snprintf(AnsiNum, sizeof(AnsiNum), FmtBuf, PrecisionLen, Val);
				}
				else
				{
					RetCnt = snprintf(AnsiNum, sizeof(AnsiNum), FmtBuf, Val);
				}
				if (RetCnt >= UE_ARRAY_COUNT(AnsiNum))
				{
					// We should print what we have written into AnsiNum but ensure we null terminate before printing
					AnsiNum[UE_ARRAY_COUNT(AnsiNum) - 1] = '\0';
					checkf(0, TEXT("Attempting to read past the size our buffer. Buffer Size: %" SIZE_T_FMT " Size to read: %d. Current contents: '%s'\n"), UE_ARRAY_COUNT(AnsiNum), RetCnt, UTF8_TO_TCHAR(AnsiNum));
				}
				if (RetCnt < 0)
				{
					return -1;
				}
				DestIter.Write(AnsiNum, RetCnt);
				break;
			}

			case 's':
			{
				if (bIsNarrowString)
				{
					ProcessStringArg<ANSICHAR>(DestIter, Src, FieldLen, PrecisionLen, ArgPtr);
				}
				else
				{
					ProcessStringArg<TCHAR>(DestIter, Src, FieldLen, PrecisionLen, ArgPtr);
				}
				break;
			}

			case 'S':
			{
				// The %S format represents a string which is the opposite of %s - wide if CharType is narrow, or narrow if CharType is wide
				using OtherCharType = std::conditional_t<std::is_same_v<CharType, ANSICHAR>, WIDECHAR, ANSICHAR>;
				ProcessStringArg<OtherCharType>(DestIter, Src, FieldLen, PrecisionLen, ArgPtr);
				break;
			}

			default:
				printf("Unknown percent [%%%c] in FGenericPlatformString::GetVarArgs().\n", *Src);
				Src++;  // skip char, I guess.
				break;
			}
		}

		int32 Result = DestIter.GetNumCharsWritten();
		return Result;
	}
}

namespace UE::Core::Private
{
	/**
	 * This is a basic object which counts how many times it has been incremented
	 */
	template <typename DestType>
	struct TCountingOutputIterator
	{
		TCountingOutputIterator()
			: Counter(0)
		{
		}

		const TCountingOutputIterator& operator* () const { return *this; }
		const TCountingOutputIterator& operator++() { ++Counter; return *this; }
		const TCountingOutputIterator& operator++(int) { ++Counter; return *this; }
		const TCountingOutputIterator& operator+=(const int32 Amount) { Counter += Amount; return *this; }

		const DestType& operator=(const DestType& Val) const
		{
			return Val;
		}

		friend int32 operator-(TCountingOutputIterator Lhs, TCountingOutputIterator Rhs)
		{
			return Lhs.Counter - Rhs.Counter;
		}

		int32 GetCount() const { return Counter; }

	private:
		int32 Counter;
	};

	/** Is the provided Codepoint within the range of valid codepoints? */
	FORCEINLINE bool IsValidCodepoint(const uint32 Codepoint)
	{
		if ((Codepoint > 0x10FFFF) ||						// No Unicode codepoints above 10FFFFh, (for now!)
			(Codepoint == 0xFFFE) || (Codepoint == 0xFFFF)) // illegal values.
		{
			return false;
		}
		return true;
	}

	/** Is the provided Codepoint within the range of the high-or low surrogates? */
	static FORCEINLINE bool IsSurrogate(const uint32 Codepoint)
	{
		return (Codepoint & 0xFFFFF800) == 0xD800;
	}

	/** Is the provided Codepoint within the range of the high-surrogates? */
	static FORCEINLINE bool IsHighSurrogate(const uint32 Codepoint)
	{
		return (Codepoint & 0xFFFFFC00) == 0xD800;
	}

	/** Is the provided Codepoint within the range of the low-surrogates? */
	static FORCEINLINE bool IsLowSurrogate(const uint32 Codepoint)
	{
		return (Codepoint & 0xFFFFFC00) == 0xDC00;
	}

	static FORCEINLINE uint32 EncodeSurrogate(const uint16 HighSurrogate, const uint16 LowSurrogate)
	{
		return ((HighSurrogate - HIGH_SURROGATE_START_CODEPOINT) << 10) + (LowSurrogate - LOW_SURROGATE_START_CODEPOINT) + 0x10000;
	}

	static FORCEINLINE void DecodeSurrogate(const uint32 Codepoint, uint16& OutHighSurrogate, uint16& OutLowSurrogate)
	{
		const uint32 TmpCodepoint = Codepoint - 0x10000;
		OutHighSurrogate = (uint16)((TmpCodepoint >> 10) + HIGH_SURROGATE_START_CODEPOINT);
		OutLowSurrogate = (TmpCodepoint & 0x3FF) + LOW_SURROGATE_START_CODEPOINT;
	}

	/** Is the provided Codepoint outside of the range of the basic multilingual plane, but within the valid range of UTF8/16? */
	static FORCEINLINE bool IsEncodedSurrogate(const uint32 Codepoint)
	{
		return Codepoint >= ENCODED_SURROGATE_START_CODEPOINT && Codepoint <= ENCODED_SURROGATE_END_CODEPOINT;
	}

	/**
	 * Convert TCHAR Codepoint into UTF-8 characters.
	 *
	 * @param Codepoint Codepoint to expand into UTF-8 bytes
	 * @param OutputIterator Output iterator to write UTF-8 bytes into
	 * @param OutputIteratorByteSizeRemaining Maximum number of ANSI characters that can be written to OutputIterator
	 * @return true if some data was written, false otherwise.
	 */
	template <typename BufferType>
	static bool WriteCodepointToBuffer(uint32 Codepoint, BufferType& OutputIterator, int32& OutputIteratorByteSizeRemaining)
	{
		// Ensure we have at least one character in size to write
		if (OutputIteratorByteSizeRemaining == 0)
		{
			return false;
		}

		if (!IsValidCodepoint(Codepoint))
		{
			Codepoint = (uint32)UNICODE_BOGUS_CHAR_CODEPOINT;
		}
		else if (IsSurrogate(Codepoint)) // UTF-8 Characters are not allowed to encode codepoints in the surrogate pair range
		{
			Codepoint = (uint32)UNICODE_BOGUS_CHAR_CODEPOINT;
		}

		// Do the encoding...
		if (Codepoint < 0x80)
		{
			*OutputIterator++ = (UTF8CHAR)Codepoint;

			OutputIteratorByteSizeRemaining -= 1;
		}
		else if (Codepoint < 0x800)
		{
			if (OutputIteratorByteSizeRemaining < 2)
			{
				return false;
			}

			*OutputIterator++ = (UTF8CHAR)((Codepoint >> 6)         | 128 | 64);
			*OutputIterator++ = (UTF8CHAR)((Codepoint       & 0x3F) | 128);

			OutputIteratorByteSizeRemaining -= 2;
		}
		else if (Codepoint < 0x10000)
		{
			if (OutputIteratorByteSizeRemaining < 3)
			{
				return false;
			}

			*OutputIterator++ = (UTF8CHAR)( (Codepoint >> 12)        | 128 | 64 | 32);
			*OutputIterator++ = (UTF8CHAR)(((Codepoint >> 6) & 0x3F) | 128);
			*OutputIterator++ = (UTF8CHAR)( (Codepoint       & 0x3F) | 128);

			OutputIteratorByteSizeRemaining -= 3;
		}
		else
		{
			if (OutputIteratorByteSizeRemaining < 4)
			{
				return false;
			}

			*OutputIterator++ = (UTF8CHAR)( (Codepoint >> 18)         | 128 | 64 | 32 | 16);
			*OutputIterator++ = (UTF8CHAR)(((Codepoint >> 12) & 0x3F) | 128);
			*OutputIterator++ = (UTF8CHAR)(((Codepoint >> 6 ) & 0x3F) | 128);
			*OutputIterator++ = (UTF8CHAR)( (Codepoint        & 0x3F) | 128);

			OutputIteratorByteSizeRemaining -= 4;
		}

		return true;
	}

	struct FNullTerminal
	{
		explicit FNullTerminal() = default;
	};

	template <typename Pointer>
	bool IsRangeEmpty(Pointer& Ptr, int32& Len)
	{
		return Len <= 0;
	}

	template <typename Pointer>
	bool IsRangeEmpty(Pointer& Ptr, FNullTerminal)
	{
		return *Ptr == '\0';
	}

	template <typename Pointer>
	void PopFront(Pointer& Ptr, int32& Len)
	{
		checkfSlow(Len > 0, TEXT("Trying to pop past end end of the range"));
		++Ptr;
		--Len;
	}

	template <typename Pointer>
	void PopFront(Pointer& Ptr, FNullTerminal)
	{
		checkfSlow(*Ptr != '\0', TEXT("Trying to pop the null terminator of the string"));
		++Ptr;
	}

	template <typename DestBufferType, typename FromType, typename SourceEndType>
	static int32 ConvertToUTF8(DestBufferType& Dest, int32 DestLen, const FromType* Source, SourceEndType SourceEnd)
	{
		using UnsignedFromType = std::make_unsigned_t<FromType>;

		if constexpr (sizeof(FromType) == 4)
		{
			DestBufferType DestStartingPosition = Dest;

			for (;;)
			{
				if (IsRangeEmpty(Source, SourceEnd))
				{
					return UE_PTRDIFF_TO_INT32(Dest - DestStartingPosition);
				}

				uint32 Codepoint = (uint32)(UnsignedFromType)*Source;
				if (!WriteCodepointToBuffer(Codepoint, Dest, DestLen))
				{
					// Could not write data, bail out
					return -1;
				}

				PopFront(Source, SourceEnd);
			}
		}
		else
		{
			DestBufferType DestStartingPosition = Dest;

			for (;;)
			{
				if (IsRangeEmpty(Source, SourceEnd))
				{
					return UE_PTRDIFF_TO_INT32(Dest - DestStartingPosition);
				}

				uint32 Codepoint = (uint32)(UnsignedFromType)*Source;
				PopFront(Source, SourceEnd);

				// Check if this character is a high-surrogate
				if (IsHighSurrogate(Codepoint))
				{
					if (IsRangeEmpty(Source, SourceEnd))
					{
						// String ends with lone high-surrogate - write it out (will be converted into bogus character)
						if (!WriteCodepointToBuffer(Codepoint, Dest, DestLen))
						{
							// Could not write data, bail out
							return -1;
						}

						return UE_PTRDIFF_TO_INT32(Dest - DestStartingPosition);
					}

					// Read next codepoint
					uint32 NextCodepoint = (uint32)(UnsignedFromType)*Source;

					// If it's a low surrogate, combine it with the current high surrogate,
					// otherwise just leave the high surrogate to be written out by itself (as a bogus character)
					if (IsLowSurrogate(NextCodepoint))
					{
						Codepoint = EncodeSurrogate((uint16)Codepoint, (uint16)NextCodepoint);
						PopFront(Source, SourceEnd);
					}
				}

				if (!WriteCodepointToBuffer(Codepoint, Dest, DestLen))
				{
					// Could not write data, bail out
					return -1;
				}
			}
		}
	}

	template <typename SourceEndType>
	static bool ReadTrailingOctet(uint32& OutOctet, const UTF8CHAR*& Ptr, SourceEndType& SourceEnd)
	{
		// Ensure our string has enough characters to read from
		if (IsRangeEmpty(Ptr, SourceEnd))
		{
			return false;
		}

		uint32 Octet = (uint32)(uint8)*Ptr;

		// Format isn't 10xxxxxx? Do not advance the pointer so the byte can be
		// re-parsed as the start of the next sequence.
		if ((Octet & 192) != 128)
		{
			return false;
		}

		PopFront(Ptr, SourceEnd);
		OutOctet = Octet;
		return true;
	};

	template <typename SourceEndType>
	static uint32 CodepointFromUtf8(const UTF8CHAR*& SourceString, SourceEndType& SourceEnd)
	{
		checkSlow(!IsRangeEmpty(SourceString, SourceEnd));

		uint32 Octet = (uint32)(uint8)*SourceString;
		PopFront(SourceString, SourceEnd);

		if (Octet < 128)  // one octet char: 0 to 127
		{
			return Octet;
		}

		if (Octet < 192)  // bad (starts with 10xxxxxx).
		{
			// Apparently each of these is supposed to be flagged as a bogus
			//  char, instead of just resyncing to the next valid codepoint.

			// Sequence was not valid UTF-8. Skip the first byte and continue.
			return UNICODE_BOGUS_CHAR_CODEPOINT;
		}

		uint32 Octet2;
		if (!ReadTrailingOctet(Octet2, SourceString, SourceEnd))
		{
			return UNICODE_BOGUS_CHAR_CODEPOINT;
		}

		if (Octet < 224)  // two octets
		{
			uint32 Codepoint = ((Octet - 192) << 6) | (Octet2 - 128);
			if (Codepoint < 0x80 || Codepoint > 0x7FF)
			{
				return UNICODE_BOGUS_CHAR_CODEPOINT;
			}

			// Should only reach here after parsing a legal codepoint
			return Codepoint;
		}

		uint32 Octet3;
		if (!ReadTrailingOctet(Octet3, SourceString, SourceEnd))
		{
			return UNICODE_BOGUS_CHAR_CODEPOINT;
		}

		if (Octet < 240)  // three octets
		{
			uint32 Codepoint = ((Octet - 224) << 12) | ((Octet2 - 128) << 6) | (Octet3 - 128);

			// UTF-8 characters cannot be in the UTF-16 surrogates range.  0xFFFE and 0xFFFF are illegal, too, so we check them at the edge.
			if (Codepoint < 0x800 || Codepoint > 0xFFFD || IsSurrogate(Codepoint))
			{
				return UNICODE_BOGUS_CHAR_CODEPOINT;
			}

			// Should only reach here after parsing a legal codepoint
			return Codepoint;
		}

		uint32 Octet4;
		if (!ReadTrailingOctet(Octet4, SourceString, SourceEnd))
		{
			return UNICODE_BOGUS_CHAR_CODEPOINT;
		}

		if (Octet < 248)  // four octets
		{
			Octet -= (128+64+32+16);

			uint32 Codepoint = (Octet << 18) | ((Octet2 - 128) << 12) | ((Octet3 - 128) << 6) | (Octet4 - 128);
			if (Codepoint < 0x10000 || Codepoint > 0x10FFFF)
			{
				return UNICODE_BOGUS_CHAR_CODEPOINT;
			}

			// Should only reach here after parsing a legal codepoint
			return Codepoint;
		}

		uint32 Octet5;
		if (!ReadTrailingOctet(Octet5, SourceString, SourceEnd))
		{
			return UNICODE_BOGUS_CHAR_CODEPOINT;
		}

		// Five and six octet sequences became illegal in rfc3629.
		//  We throw the codepoint away, but parse them to make sure we move
		//  ahead the right number of bytes and don't overflow the buffer.

		if (Octet < 252)  // five octets
		{
			// Skip to end and write out a single char (we always have room for at least 1 char)
			return UNICODE_BOGUS_CHAR_CODEPOINT;
		}

		// six octets

		uint32 Octet6;
		if (!ReadTrailingOctet(Octet6, SourceString, SourceEnd))
		{
			return UNICODE_BOGUS_CHAR_CODEPOINT;
		}

		return UNICODE_BOGUS_CHAR_CODEPOINT;
	}

	/**
	 * Read Source string, converting the data from UTF-8 into UTF-16, and placing these in the Destination
	 */
	template <typename DestType, typename DestBufferType, typename SourceEndType>
	static int32 ConvertFromUTF8(DestBufferType& ConvertedBuffer, int32 DestLen, const UTF8CHAR* Source, SourceEndType SourceEnd)
	{
		check(DestLen >= 0);

		DestBufferType DestStartingPosition = ConvertedBuffer;

		const uint64 ExtendedCharMask = 0x8080808080808080;
		while (!IsRangeEmpty(Source, SourceEnd))
		{
			if (DestLen == 0)
			{
				return -1;
			}

			// In case we're given an unaligned pointer, we'll
			// fallback to the slow path until properly aligned.
			// But we can only do that if we know how much buffer we have left.
			if constexpr (std::is_integral_v<SourceEndType>)
			{
				if (IsAligned(Source, 8))
				{
					// Fast path for most common case
					while (SourceEnd >= 8 && DestLen >= 8)
					{
						// Detect any extended characters 8 chars at a time
						if ((*(const uint64*)Source) & ExtendedCharMask)
						{
							// Move to slow path since we got extended characters to process
							break;
						}

						// This should get unrolled on most compiler
						// ROI of diminished return to vectorize this as we 
						// would have to deal with alignment, endianness and
						// rewrite the iterators to support bulk writes
						for (int32 Index = 0; Index < 8; ++Index)
						{
							*(ConvertedBuffer++) = (DestType)(uint8)*(Source++);
						}
						SourceEnd -= 8;
						DestLen -= 8;
					}
				}
			}

			// Slow path for extended characters
			while (!IsRangeEmpty(Source, SourceEnd) && DestLen > 0)
			{
				// Read our codepoint, advancing the source pointer
				uint32 Codepoint = CodepointFromUtf8(Source, SourceEnd);

				if constexpr (sizeof(DestType) != 4)
				{
					// We want to write out two chars
					if (IsEncodedSurrogate(Codepoint))
					{
						// We need two characters to write the surrogate pair
						if (DestLen >= 2)
						{
							uint16 HighSurrogate = 0;
							uint16 LowSurrogate = 0;
							DecodeSurrogate(Codepoint, HighSurrogate, LowSurrogate);

							*(ConvertedBuffer++) = (DestType)HighSurrogate;
							*(ConvertedBuffer++) = (DestType)LowSurrogate;
							DestLen -= 2;
							continue;
						}

						// If we don't have space, write a bogus character instead (we should have space for it)
						Codepoint = (uint32)UNICODE_BOGUS_CHAR_CODEPOINT;
					}
					else if (Codepoint > ENCODED_SURROGATE_END_CODEPOINT)
					{
						// Ignore values higher than the supplementary plane range
						Codepoint = (uint32)UNICODE_BOGUS_CHAR_CODEPOINT;
					}
				}

				*(ConvertedBuffer++) = (DestType)Codepoint;
				--DestLen;

				if constexpr (std::is_integral_v<SourceEndType>)
				{
					// Return to the fast path once aligned and back to simple ASCII chars
					if (Codepoint < 128 && IsAligned(Source, 8))
					{
						break;
					}
				}
			}
		}

		return UE_PTRDIFF_TO_INT32(ConvertedBuffer - DestStartingPosition);
	}

	/**
	 * Determines the length of the converted string.
	 *
	 * @return The length of the string in UTF-16 code units.
	 */
	int32 GetConvertedLength(const UTF8CHAR*, const WIDECHAR* Source)
	{
		TCountingOutputIterator<UTF8CHAR> Dest;
		int32 Result = ConvertToUTF8(Dest, INT32_MAX, Source, FNullTerminal{});
		return Result;
	}
	int32 GetConvertedLength(const UTF8CHAR*, const WIDECHAR* Source, int32 SourceLen)
	{
		TCountingOutputIterator<UTF8CHAR> Dest;
		int32 Result = ConvertToUTF8(Dest, INT32_MAX, Source, SourceLen);
		return Result;
	}
	int32 GetConvertedLength(const UTF8CHAR*, const UCS2CHAR* Source)
	{
		TCountingOutputIterator<UTF8CHAR> Dest;
		int32 Result = ConvertToUTF8(Dest, INT32_MAX, Source, FNullTerminal{});
		return Result;
	}
	int32 GetConvertedLength(const UTF8CHAR*, const UCS2CHAR* Source, int32 SourceLen)
	{
		TCountingOutputIterator<UTF8CHAR> Dest;
		int32 Result = ConvertToUTF8(Dest, INT32_MAX, Source, SourceLen);
		return Result;
	}
	int32 GetConvertedLength(const UTF8CHAR*, const UTF32CHAR* Source)
	{
		TCountingOutputIterator<UTF8CHAR> Dest;
		int32 Result = ConvertToUTF8(Dest, INT32_MAX, Source, FNullTerminal{});
		return Result;
	}
	int32 GetConvertedLength(const UTF8CHAR*, const UTF32CHAR* Source, int32 SourceLen)
	{
		TCountingOutputIterator<UTF8CHAR> Dest;
		int32 Result = ConvertToUTF8(Dest, INT32_MAX, Source, SourceLen);
		return Result;
	}
	int32 GetConvertedLength(const ANSICHAR*, const UTF8CHAR* Source)
	{
		TCountingOutputIterator<ANSICHAR> Dest;
		int32 Result = ConvertFromUTF8<ANSICHAR>(Dest, INT32_MAX, Source, FNullTerminal{});
		return Result;
	}
	int32 GetConvertedLength(const ANSICHAR*, const UTF8CHAR* Source, int32 SourceLen)
	{
		TCountingOutputIterator<ANSICHAR> Dest;
		int32 Result = ConvertFromUTF8<ANSICHAR>(Dest, INT32_MAX, Source, SourceLen);
		return Result;
	}
	int32 GetConvertedLength(const WIDECHAR*, const UTF8CHAR* Source)
	{
		TCountingOutputIterator<WIDECHAR> Dest;
		int32 Result = ConvertFromUTF8<WIDECHAR>(Dest, INT32_MAX, Source, FNullTerminal{});
		return Result;
	}
	int32 GetConvertedLength(const WIDECHAR*, const UTF8CHAR* Source, int32 SourceLen)
	{
		TCountingOutputIterator<WIDECHAR> Dest;
		int32 Result = ConvertFromUTF8<WIDECHAR>(Dest, INT32_MAX, Source, SourceLen);
		return Result;
	}
	int32 GetConvertedLength(const UCS2CHAR*, const UTF8CHAR* Source)
	{
		TCountingOutputIterator<UCS2CHAR> Dest;
		int32 Result = ConvertFromUTF8<UCS2CHAR>(Dest, INT32_MAX, Source, FNullTerminal{});
		return Result;
	}
	int32 GetConvertedLength(const UCS2CHAR*, const UTF8CHAR* Source, int32 SourceLen)
	{
		TCountingOutputIterator<UCS2CHAR> Dest;
		int32 Result = ConvertFromUTF8<UCS2CHAR>(Dest, INT32_MAX, Source, SourceLen);
		return Result;
	}

	UTF8CHAR* Convert(UTF8CHAR* Dest, int32 DestLen, const WIDECHAR* Src)
	{
		if (ConvertToUTF8(Dest, DestLen, Src, FNullTerminal{}) == -1)
		{
			return nullptr;
		}
		return Dest;
	}
	UTF8CHAR* Convert(UTF8CHAR* Dest, int32 DestLen, const WIDECHAR* Src, int32 SrcLen)
	{
		if (ConvertToUTF8(Dest, DestLen, Src, SrcLen) == -1)
		{
			return nullptr;
		}
		return Dest;
	}
	UTF8CHAR* Convert(UTF8CHAR* Dest, int32 DestLen, const UCS2CHAR* Src)
	{
		if (ConvertToUTF8(Dest, DestLen, Src, FNullTerminal{}) == -1)
		{
			return nullptr;
		}
		return Dest;
	}
	UTF8CHAR* Convert(UTF8CHAR* Dest, int32 DestLen, const UCS2CHAR* Src, int32 SrcLen)
	{
		if (ConvertToUTF8(Dest, DestLen, Src, SrcLen) == -1)
		{
			return nullptr;
		}
		return Dest;
	}
	UTF8CHAR* Convert(UTF8CHAR* Dest, int32 DestLen, const UTF32CHAR* Src)
	{
		if (ConvertToUTF8(Dest, DestLen, Src, FNullTerminal{}) == -1)
		{
			return nullptr;
		}
		return Dest;
	}
	UTF8CHAR* Convert(UTF8CHAR* Dest, int32 DestLen, const UTF32CHAR* Src, int32 SrcLen)
	{
		if (ConvertToUTF8(Dest, DestLen, Src, SrcLen) == -1)
		{
			return nullptr;
		}
		return Dest;
	}
	ANSICHAR* Convert(ANSICHAR* Dest, int32 DestLen, const UTF8CHAR* Src)
	{
		if (ConvertFromUTF8<ANSICHAR>(Dest, DestLen, Src, FNullTerminal{}) == -1)
		{
			return nullptr;
		}
		return Dest;
	}
	ANSICHAR* Convert(ANSICHAR* Dest, int32 DestLen, const UTF8CHAR* Src, int32 SrcLen)
	{
		if (ConvertFromUTF8<ANSICHAR>(Dest, DestLen, Src, SrcLen) == -1)
		{
			return nullptr;
		}
		return Dest;
	}
	WIDECHAR* Convert(WIDECHAR* Dest, int32 DestLen, const UTF8CHAR* Src)
	{
		if (ConvertFromUTF8<WIDECHAR>(Dest, DestLen, Src, FNullTerminal{}) == -1)
		{
			return nullptr;
		}
		return Dest;
	}
	WIDECHAR* Convert(WIDECHAR* Dest, int32 DestLen, const UTF8CHAR* Src, int32 SrcLen)
	{
		if (ConvertFromUTF8<WIDECHAR>(Dest, DestLen, Src, SrcLen) == -1)
		{
			return nullptr;
		}
		return Dest;
	}
	UCS2CHAR* Convert(UCS2CHAR* Dest, int32 DestLen, const UTF8CHAR* Src)
	{
		if (ConvertFromUTF8<UCS2CHAR>(Dest, DestLen, Src, FNullTerminal{}) == -1)
		{
			return nullptr;
		}
		return Dest;
	}
	UCS2CHAR* Convert(UCS2CHAR* Dest, int32 DestLen, const UTF8CHAR* Src, int32 SrcLen)
	{
		if (ConvertFromUTF8<UCS2CHAR>(Dest, DestLen, Src, SrcLen) == -1)
		{
			return nullptr;
		}
		return Dest;
	}
}

namespace GenericPlatformStringPrivate
{

template<typename CharType1, typename CharType2>
int32 StrncmpImpl(const CharType1* String1, const CharType2* String2, SIZE_T Count)
{
	for (; Count > 0; --Count)
	{
		uint32 AsUnsignedChar1 = TChar<CharType1>::ToUnsigned(*String1++);
		uint32 AsUnsignedChar2 = TChar<CharType2>::ToUnsigned(*String2++);

		if (AsUnsignedChar1 != AsUnsignedChar2)
		{
			return AsUnsignedChar1 - AsUnsignedChar2;
		}
		if (AsUnsignedChar1 == 0)
		{
			return 0;
		}
	}

	return 0;
}

}

int32 FGenericPlatformString::FormatV(ANSICHAR* Dest, SIZE_T DestSize, const ANSICHAR* Fmt, va_list ArgPtr)
{
	return GetVarArgsImpl(Dest, DestSize, Fmt, ArgPtr);
}

int32 FGenericPlatformString::FormatV(WIDECHAR* Dest, SIZE_T DestSize, const WIDECHAR* Fmt, va_list ArgPtr)
{
	return GetVarArgsImpl(Dest, DestSize, Fmt, ArgPtr);
}

int32 FGenericPlatformString::Strncmp(const ANSICHAR* Str1, const ANSICHAR* Str2, SIZE_T Count) { return GenericPlatformStringPrivate::StrncmpImpl(Str1, Str2, Count); }
int32 FGenericPlatformString::Strncmp(const WIDECHAR* Str1, const ANSICHAR* Str2, SIZE_T Count) { return GenericPlatformStringPrivate::StrncmpImpl(Str1, Str2, Count); }
int32 FGenericPlatformString::Strncmp(const UTF8CHAR* Str1, const ANSICHAR* Str2, SIZE_T Count) { return GenericPlatformStringPrivate::StrncmpImpl(Str1, Str2, Count); }
int32 FGenericPlatformString::Strncmp(const ANSICHAR* Str1, const WIDECHAR* Str2, SIZE_T Count) { return GenericPlatformStringPrivate::StrncmpImpl(Str1, Str2, Count); }
int32 FGenericPlatformString::Strncmp(const WIDECHAR* Str1, const WIDECHAR* Str2, SIZE_T Count) { return GenericPlatformStringPrivate::StrncmpImpl(Str1, Str2, Count); }
int32 FGenericPlatformString::Strncmp(const UTF8CHAR* Str1, const WIDECHAR* Str2, SIZE_T Count) { return GenericPlatformStringPrivate::StrncmpImpl(Str1, Str2, Count); }
int32 FGenericPlatformString::Strncmp(const ANSICHAR* Str1, const UTF8CHAR* Str2, SIZE_T Count) { return GenericPlatformStringPrivate::StrncmpImpl(Str1, Str2, Count); }
int32 FGenericPlatformString::Strncmp(const WIDECHAR* Str1, const UTF8CHAR* Str2, SIZE_T Count) { return GenericPlatformStringPrivate::StrncmpImpl(Str1, Str2, Count); }
int32 FGenericPlatformString::Strncmp(const UTF8CHAR* Str1, const UTF8CHAR* Str2, SIZE_T Count) { return GenericPlatformStringPrivate::StrncmpImpl(Str1, Str2, Count); }

#if WITH_LOW_LEVEL_TESTS

#include "Containers/AnsiString.h"
#include "Containers/Utf8String.h"
#include "TestHarness.h"
#include <catch2/generators/catch_generators.hpp>

template <int32 OutputSize, typename ExpectedCharType, typename FormatCharType>
static void TestFormatV(const ExpectedCharType* Expected, const FormatCharType* Format, ...)
{
	FormatCharType OutputString[OutputSize];

	int32 Result;
	{
		va_list ArgPtr;
		va_start(ArgPtr, Format);
		Result = FGenericPlatformString::FormatV(OutputString, OutputSize, Format, ArgPtr);
		va_end(ArgPtr);
	}

	TString<ExpectedCharType> ExpectedString(Expected);
	CAPTURE(ExpectedString);
	CHECK(ExpectedString.Len() == Result);
	CHECK(ExpectedString.Left(OutputSize - 1) == TString<ExpectedCharType>(OutputString));
	CHECK(OutputString[FPlatformMath::Min(Result, OutputSize - 1)] == 0);

	if (!ExpectedString.IsEmpty())
	{
		static_assert(OutputSize > 1);
		OutputString[0] = '.';
		OutputString[1] = '.';

		va_list ArgPtr;
		va_start(ArgPtr, Format);
		Result = FGenericPlatformString::FormatV(OutputString, 1, Format, ArgPtr);
		va_end(ArgPtr);

		CHECK(ExpectedString.Len() == Result);
		CHECK(OutputString[0] == 0);
		CHECK(OutputString[1] == '.');
	}
}

template <typename CharType>
static int32 TestFormat(CharType* Dest, SIZE_T DestSize, const CharType* Format, ...)
{
	va_list Args;
	va_start(Args, Format);
	int32 Result = FGenericPlatformString::FormatV(Dest, DestSize, Format, Args);
	va_end(Args);
	return Result;
}

template <typename CharType>
static int32 TestGetVarArgs(CharType* Dest, SIZE_T DestSize, const CharType* Format, ...)
{
	va_list Args;
	va_start(Args, Format);
	int32 Result = FGenericPlatformString::GetVarArgs(Dest, DestSize, Format, Args);
	va_end(Args);
	return Result;
}

TEST_CASE("Core::GenericPlatformString::FormatV", "[Core][String][Smoke]")
{
	constexpr int32 OutputSize = 256;

	TestFormatV<OutputSize>(TEXT("Test A|LEFT                |               RIGHT|     33.33|66.67     |"), TEXT("Test A|%-20s|%20s|%10.2f|%-10.2f|"), TEXT("LEFT"), TEXT("RIGHT"), 33.333333, 66.666666);
	TestFormatV<OutputSize>(TEXT("Test B|Percents:%%%3|"), TEXT("Test B|Percents:%%%%%%%d|"), 3);
	TestFormatV<OutputSize>(TEXT("Test C|12345|54321|123ABC|f|99|"), TEXT("Test C|%d|%i|%X|%x|%u|"), 12345, 54321, 0x123AbC, 15, 99);

#ifdef _MSC_VER
	// MSVC's standard library formats pointers differently
	TestFormatV<OutputSize>(TEXT("Test D|0000000000012345|"), TEXT("Test D|%p|"), 0x12345);
#else
	// Pointer format (which we get from snprintf) varies from platform to platform.
	// Call it to make sure it puts the output in the proper place, but don't test for the exact format
	{
		WIDECHAR Output[OutputSize];
		TCString<WIDECHAR>::Snprintf(Output, OutputSize, WIDETEXT("Test D|%p|"), 0x12345);
		FString OutputString(Output);
		CHECK_MESSAGE(Output, OutputString.StartsWith(TEXT("Test D|"))
			&& OutputString.EndsWith(TEXT("|")) && OutputString.Len() < TEXTVIEW("Test D||").Len() + 20);
	}
#endif

	TestFormatV<OutputSize>(TEXT("Test E|12345678912345|"), TEXT("Test E|%" INT64_FMT "|"), int64(12345678912345LL));
	TestFormatV<OutputSize>(TEXT("Test F|123.456000|1.234560e+02|123.456|"), TEXT("Test F|%f|%e|%g|"), 123.456, 123.456, 123.456);
	TestFormatV<OutputSize>(TEXT("Test G|C0DE|"), TEXT("Test G|%" UPTRINT_X_FMT"|"), UPTRINT(49374));
	TestFormatV<OutputSize>(TEXT("Test H|c0de|"), TEXT("Test H|%" UPTRINT_x_FMT "|"), UPTRINT(49374));
	TestFormatV<OutputSize>(TEXT("Test I|18446744073709551615|"), TEXT("Test I|%" UINT64_FMT "|"), MAX_uint64);
	TestFormatV<OutputSize>(TEXT("Test J|         J|"), TEXT("Test J|%*c|"), 10, 'J');
	TestFormatV<OutputSize>(TEXT("Test K|K    |"), TEXT("Test K|%-5c|"), 'K');
	TestFormatV<OutputSize>(TEXT("Test L|Hello World|"), TEXT("Test L|%hs|"), UTF8TEXT("Hello World"));
	TestFormatV<OutputSize>(TEXT("Test M|\U0001F606\U0001F41C|"), TEXT("Test M|%hs|"), UTF8TEXT("\U0001F606\U0001F41C"));
	TestFormatV<OutputSize>(TEXT("Test N|\U0001F606\U0001F41C|"), TEXT("Test N|%ls|"), TEXT("\U0001F606\U0001F41C"));
	TestFormatV<OutputSize>(TEXT("Test O|\U0001F606\U0001F41C|"), TEXT("Test O|%s|"), TEXT("\U0001F606\U0001F41C"));
	TestFormatV<OutputSize>(TEXT("Test P|\U0001F606\U0001F41C|"), TEXT("Test P|%S|"), UTF8TEXT("\U0001F606\U0001F41C"));

	TestFormatV<9>(TEXT("12345678"), TEXT("%08x"), 0x12345678);

	TestFormatV<OutputSize>("Test A|LEFT                |               RIGHT|     33.33|66.67     |", "Test A|%-20s|%20s|%10.2f|%-10.2f|", "LEFT", "RIGHT", 33.333333, 66.666666);
	TestFormatV<OutputSize>("Test B|Percents:%%%3|", "Test B|Percents:%%%%%%%d|", 3);
	TestFormatV<OutputSize>("Test C|12345|54321|123ABC|f|99|", "Test C|%d|%i|%X|%x|%u|", 12345, 54321, 0x123AbC, 15, 99);

#ifdef _MSC_VER
	// MSVC's standard library formats pointers differently
	TestFormatV<OutputSize>("Test D|0000000000012345|", "Test D|%p|", 0x12345);
#else
	// Pointer format (which we get from snprintf) varies from platform to platform.
	// Call it to make sure it puts the output in the proper place, but don't test for the exact format
	{
		ANSICHAR Output[OutputSize];
		TCString<ANSICHAR>::Snprintf(Output, OutputSize, "Test D|%p|", 0x12345);
		FAnsiString OutputString(Output);
		CHECK_MESSAGE(Output, OutputString.StartsWith(TEXT("Test D|"))
			&& OutputString.EndsWith(TEXT("|")) && OutputString.Len() < TEXTVIEW("Test D||").Len() + 20);
	}
#endif

	TestFormatV<OutputSize>("Test E|12345678912345|", "Test E|%" INT64_FMT "|", int64(12345678912345LL));
	TestFormatV<OutputSize>("Test F|123.456000|1.234560e+02|123.456|", "Test F|%f|%e|%g|", 123.456, 123.456, 123.456);
	TestFormatV<OutputSize>("Test G|C0DE|", "Test G|%" UPTRINT_X_FMT"|", UPTRINT(49374));
	TestFormatV<OutputSize>("Test H|c0de|", "Test H|%" UPTRINT_x_FMT "|", UPTRINT(49374));
	TestFormatV<OutputSize>("Test I|18446744073709551615|", "Test I|%" UINT64_FMT "|", MAX_uint64);
	TestFormatV<OutputSize>("Test J|         J|", "Test J|%*c|", 10, 'J');
	TestFormatV<OutputSize>("Test K|K    |", "Test K|%-5c|", 'K');
	TestFormatV<OutputSize>("Test L|Hello World|", "Test L|%hs|", UTF8TEXT("Hello World"));
	TestFormatV<OutputSize>(UTF8TEXT("Test M|\U0001F606\U0001F41C|"), "Test M|%ls|", TEXT("\U0001F606\U0001F41C"));
	TestFormatV<OutputSize>(UTF8TEXT("Test N|\U0001F606\U0001F41C|"), "Test N|%hs|", UTF8TEXT("\U0001F606\U0001F41C"));
	TestFormatV<OutputSize>(UTF8TEXT("Test O|\U0001F606\U0001F41C|"), "Test O|%s|", UTF8TEXT("\U0001F606\U0001F41C"));
	TestFormatV<OutputSize>(UTF8TEXT("Test P|\U0001F606\U0001F41C|"), "Test P|%S|", TEXT("\U0001F606\U0001F41C"));

	TestFormatV<9>("12345678", "%08x", 0x12345678);

	constexpr uint8 Sentinel = 0xcc;
	const auto CheckAllSentinel = [Sentinel]<typename T, int32 N>(T (&Array)[N]) -> bool
	{
		bool bOk = true;
		for (uint8 Value : MakeArrayView((const uint8*)Array, N * sizeof(T)))
		{
			bOk &= Value == Sentinel;
		}
		return bOk;
	};

	{
		WIDECHAR Output[OutputSize];
		FMemory::Memset(Output, Sentinel, OutputSize * sizeof(WIDECHAR));

		CHECK(TestFormat((WIDECHAR*)nullptr, 0, WIDETEXT("")) == 0);
		CHECK(TestFormat(Output + 8, 0, WIDETEXT("")) == 0);
		CHECK(CheckAllSentinel(Output));
		CHECK(TestFormat((WIDECHAR*)nullptr, 0, WIDETEXT("%d"), 1) == 1);
		CHECK(TestFormat(Output + 8, 0, WIDETEXT("%d"), 1) == 1);
		CHECK(CheckAllSentinel(Output));
	}

	{
		ANSICHAR Output[OutputSize];
		FMemory::Memset(Output, Sentinel, OutputSize * sizeof(ANSICHAR));

		CHECK(TestFormat((ANSICHAR*)nullptr, 0, "") == 0);
		CHECK(TestFormat(Output + 8, 0, "") == 0);
		CHECK(CheckAllSentinel(Output));
		CHECK(TestFormat((ANSICHAR*)nullptr, 0, "%d", 1) == 1);
		CHECK(TestFormat(Output + 8, 0, "%d", 1) == 1);
		CHECK(CheckAllSentinel(Output));
	}
}

TEST_CASE("Core::GenericPlatformString::GetVarArgs", "[Core][String][Smoke]")
{
	// GetVarArgs is a wrapper for FormatV. Test only the expected difference in return value.
	TCHAR Output[16];
	CHECK(TestGetVarArgs(Output, 0, TEXT("")) == -1);
	CHECK(TestGetVarArgs(Output, 1, TEXT("")) == 0);
	CHECK(TestGetVarArgs(Output, 0, TEXT("%d"), 1) == -1);
	CHECK(TestGetVarArgs(Output, 1, TEXT("%d"), 1) == -1);
	CHECK(TestGetVarArgs(Output, 2, TEXT("%d"), 1) == 1);
	CHECK(TestGetVarArgs(Output, 3, TEXT("%d"), 1) == 1);
}
#endif // WITH_LOW_LEVEL_TESTS