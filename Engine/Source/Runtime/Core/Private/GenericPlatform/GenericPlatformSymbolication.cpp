// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenericPlatform/GenericPlatformSymbolication.h"
#include "Logging/LogMacros.h"
#include "HAL/FileManager.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Serialization/ArchiveSaveCompressedProxy.h"
#include "Serialization/ArchiveLoadCompressedProxy.h"
#include "Misc/Compression.h"


DEFINE_LOG_CATEGORY_STATIC(LogGenericPlatformSymbolication, Log, All);

bool FGenericPlatformSymbolication::LoadSymbolDatabaseForBinary(FString SourceFolder, FString Binary, FString ModuleSignature, TOptional<FString> Architecture, FGenericPlatformSymbolDatabase& OutDatabase)
{
	bool bOk = false;
	
	FString ModuleName = FPaths::GetBaseFilename(Binary);
	FString InputFile = (SourceFolder / ModuleName) + TEXT(".udebugsymbols");
	if (IFileManager::Get().FileSize(*InputFile) > 0)
	{
		TArray<uint8> DataBuffer;
		if(FFileHelper::LoadFileToArray(DataBuffer, *InputFile))
		{
			FArchiveLoadCompressedProxy DataArchive(DataBuffer, NAME_Zlib);
			DataArchive << OutDatabase;
			if(!DataArchive.GetError())
			{
				if(OutDatabase.Signature == ModuleSignature)
				{
					bOk = true;
				}
			}
		}
	}
	
	return bOk;
}

bool FGenericPlatformSymbolication::SaveSymbolDatabaseForBinary(FString TargetFolder, FString Name, FGenericPlatformSymbolDatabase& Database)
{
	bool bOk = false;
	
	FString ModuleName = FPaths::GetBaseFilename(Name);
	FString OutputFile = (TargetFolder / ModuleName) + TEXT(".udebugsymbols");

	TArray<uint8> DataBuffer;
	FArchiveSaveCompressedProxy DataArchive(DataBuffer, NAME_Zlib, COMPRESS_BiasSpeed);
	DataArchive << Database;
	DataArchive.Flush();
	if(!DataArchive.GetError())
	{
		if(FFileHelper::SaveArrayToFile(DataBuffer, *OutputFile))
		{
			bOk = true;
		}
		else
		{
			UE_LOGF( LogGenericPlatformSymbolication, Error, "Unable to write debug symbols to output file." );
		}
	}
	else
	{
		UE_LOGF( LogGenericPlatformSymbolication, Error, "Unable to serialise debug symbols." );
	}
	
	return bOk;
}

bool FGenericPlatformSymbolication::SymbolInfoForStrippedSymbol(FGenericPlatformSymbolDatabase const& Database, uint64 ProgramCounter, uint64 ModuleOffset, FString ModuleSignature, FProgramCounterSymbolInfo& Info)
{
	bool bOk = false;
	
	if(Database.Signature == ModuleSignature)
	{
		for(auto Symbol : Database.Symbols)
		{
			if((Symbol.Start <= (ProgramCounter - ModuleOffset)) && ((Symbol.Start + Symbol.Length) >= (ProgramCounter - ModuleOffset)))
			{
				FCStringAnsi::Strncpy(Info.ModuleName, TCHAR_TO_ANSI(*Database.Name), FProgramCounterSymbolInfo::MAX_NAME_LENGTH);
				
				FString SymbolName = Database.StringTable[Symbol.NameIdx];
				FCStringAnsi::Strncpy(Info.FunctionName, TCHAR_TO_ANSI(*SymbolName), FProgramCounterSymbolInfo::MAX_NAME_LENGTH);
				
				Info.ProgramCounter = ProgramCounter;
				Info.OffsetInModule = Symbol.Start;
				
				Info.LineNumber = -1;
				
				for(auto SymbolInfo : Symbol.SymbolInfo)
				{
					if((SymbolInfo.Start <= (ProgramCounter - ModuleOffset)) && ((SymbolInfo.Start + SymbolInfo.Length) >= (ProgramCounter - ModuleOffset)))
					{
						FString Path = Database.StringTable[SymbolInfo.PathIdx];
						FCStringAnsi::Strncpy(Info.Filename, TCHAR_TO_ANSI(*Path), FProgramCounterSymbolInfo::MAX_NAME_LENGTH);
						Info.LineNumber = SymbolInfo.Line;
						break;
					}
				}
				
				bOk = true;
			
				break;
			}
		}
	}
	
	return bOk;
}
