// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenericPlatform/GenericPlatformTime.h"
#include "HAL/PlatformTime.h"
#include "Logging/LogMacros.h"
#include "Misc/Parse.h"
#include "Delegates/IDelegateInstance.h"
#include "Containers/Ticker.h"
#include "HAL/IConsoleManager.h"
#include "Stats/Stats.h"

DEFINE_LOG_CATEGORY_STATIC(LogGenericPlatformTime, Log, All);

double FGenericPlatformTime::SecondsPerCycle = 0.0;	
double FGenericPlatformTime::SecondsPerCycle64 = 0.0;
double FGenericPlatformTime::LastIntervalCPUTimeInSeconds = 0.0;

#if PLATFORM_HAS_BSD_TIME 

#include <sched.h>

double FGenericPlatformTime::InitTiming()
{
	// we use gettimeofday() instead of rdtsc, so it's 1000000 "cycles" per second on this faked CPU.
	SecondsPerCycle = 1.0f / 1000000.0f;
	SecondsPerCycle64 = 1.0 / 1000000.0;
	return FPlatformTime::Seconds();
}

void FGenericPlatformTime::SystemTime( int32& Year, int32& Month, int32& DayOfWeek, int32& Day, int32& Hour, int32& Min, int32& Sec, int32& MSec )
{
	// query for calendar time
	struct timeval Time;
	gettimeofday(&Time, NULL);

	// convert it to local time
	struct tm LocalTime;
	localtime_r(&Time.tv_sec, &LocalTime);

	// pull out data/time
	Year		= LocalTime.tm_year + 1900;
	Month		= LocalTime.tm_mon + 1;
	DayOfWeek	= LocalTime.tm_wday;
	Day			= LocalTime.tm_mday;
	Hour		= LocalTime.tm_hour;
	Min			= LocalTime.tm_min;
	Sec			= LocalTime.tm_sec;
	MSec		= Time.tv_usec / 1000;
}

void FGenericPlatformTime::UtcTime( int32& Year, int32& Month, int32& DayOfWeek, int32& Day, int32& Hour, int32& Min, int32& Sec, int32& MSec )
{
	// query for calendar time
	struct timeval Time;
	gettimeofday(&Time, NULL);

	// convert it to UTC
	struct tm LocalTime;
	gmtime_r(&Time.tv_sec, &LocalTime);

	// pull out data/time
	Year		= LocalTime.tm_year + 1900;
	Month		= LocalTime.tm_mon + 1;
	DayOfWeek	= LocalTime.tm_wday;
	Day			= LocalTime.tm_mday;
	Hour		= LocalTime.tm_hour;
	Min			= LocalTime.tm_min;
	Sec			= LocalTime.tm_sec;
	MSec		= Time.tv_usec / 1000;
}

#endif

TCHAR* FGenericPlatformTime::StrDate( TCHAR* Dest, SIZE_T DestSize )
{
	int32 Year;
	int32 Month;
	int32 DayOfWeek;
	int32 Day;
	int32 Hour;
	int32 Min;
	int32 Sec;
	int32 MSec;

	FPlatformTime::SystemTime(Year, Month, DayOfWeek, Day, Hour, Min, Sec, MSec );
	FCString::Snprintf(Dest, (int32)DestSize, TEXT("%02d/%02d/%02d"), Month, Day, Year % 100);
	return Dest;
}

TCHAR* FGenericPlatformTime::StrTime( TCHAR* Dest, SIZE_T DestSize )
{
	int32 Year;
	int32 Month;
	int32 DayOfWeek;
	int32 Day;
	int32 Hour;
	int32 Min;
	int32 Sec;
	int32 MSec;

	FPlatformTime::SystemTime(Year, Month, DayOfWeek, Day, Hour, Min, Sec, MSec );
	FCString::Snprintf( Dest, (int32)DestSize, TEXT("%02d:%02d:%02d"), Hour, Min, Sec);
	return Dest;
}

const TCHAR* FGenericPlatformTime::StrTimestamp()
{
	static thread_local TCHAR Result[1024];
	*Result = TEXT('\0');
	StrDate( Result, UE_ARRAY_COUNT(Result) );
	FCString::Strcat( Result, TEXT(" ") );
	StrTime( Result + FCString::Strlen(Result), UE_ARRAY_COUNT(Result) - FCString::Strlen(Result) );
	return Result;
}

/**
 * Returns a pretty-string for a time given in seconds. (I.e. "4:31 min", "2:16:30 hours", etc)
 * @param Seconds	Time in seconds
 * @return			Time in a pretty formatted string
 */
FString FGenericPlatformTime::PrettyTime( double Seconds )
{
	if (Seconds < 0.001)
	{
		return FString::Printf( TEXT("%d us"), FMath::TruncToInt((float)(Seconds*1000000)) );
	}
	else if ( Seconds < 1.0 )
	{
		int32 Ms = FMath::TruncToInt((float)(Seconds * 1000));
		int32 Us = FMath::TruncToInt((float)(Seconds * 1000000)) - Ms*1000;
		return FString::Printf( TEXT("%d.%03d ms"), Ms, Us );
	}
	else if ( Seconds < 10.0 )
	{
		int32 Sec = FMath::TruncToInt((float)Seconds);
		int32 Ms = FMath::TruncToInt((float)(Seconds*1000)) - Sec*1000;
		return FString::Printf( TEXT("%d.%02d sec"), Sec, Ms/10 );
	}
	else if ( Seconds < 60.0 )
	{
		int32 Sec = FMath::TruncToInt((float)Seconds);
		int32 Ms = FMath::TruncToInt((float)(Seconds*1000)) - Sec*1000;
		return FString::Printf( TEXT("%d.%d sec"), Sec, Ms/100 );
	}
	else if ( Seconds < 60.0*60.0 )
	{
		int32 Min = FMath::TruncToInt((float)(Seconds/60.0));
		int32 Sec = FMath::TruncToInt((float)Seconds) - Min*60;
		return FString::Printf( TEXT("%d:%02d min"), Min, Sec );
	}
	else
	{
		int32 Hr = FMath::TruncToInt((float)(Seconds/3600.0));
		int32 Min = FMath::TruncToInt((float)((Seconds - Hr*3600)/60.0));
		int32 Sec = FMath::TruncToInt((float)(Seconds - Hr*3600 - Min*60));
		return FString::Printf( TEXT("%d:%02d:%02d hours"), Hr, Min, Sec );
	}
}

/** Implements CPUTimeDumpCommand command. */
struct FCPUTimeDump
{
	/** Default constructor. */
	FCPUTimeDump()
		: GetCPUTimeDelegate( FTickerDelegate::CreateRaw( this, &FCPUTimeDump::OnAddCPUTime ) )
		, CPUTimeDumpDelegate( FTickerDelegate::CreateRaw( this, &FCPUTimeDump::OnCPUTimeDump ) )
	{}

	/** Gets FCPUTimeDump singleton instance. */
	static FCPUTimeDump& Get()
	{
		static FCPUTimeDump Singleton;
		return Singleton;
	}

	/** Executes the CPUTime.Dump command. */
	void ExecuteCommand( const TArray< FString >& Args )
	{
		int32 Delay = 0;
		if( Args.Num() == 0 )
		{
			Delay = 30;
		}
		else
		{
			FParse::Value(*Args[0], TEXT("delay="), Delay);
		}

		FTSTicker::RemoveTicker( GetCPUTimeDelegateHandle );
		FTSTicker::RemoveTicker( CPUTimeDumpDelegateHandle );
		GetCPUTimeDelegateHandle .Reset();
		CPUTimeDumpDelegateHandle.Reset();

		if( Delay == 0 )
		{
			UE_LOGF(LogGenericPlatformTime, Log, "Disabling printing the CPU usage");
		}
		else
		{
			Delay = FMath::Clamp( Delay, 10, 300 );
			UE_LOGF(LogGenericPlatformTime, Log, "Delay set to %i second(s), started printing the CPU usage", Delay);

			GetCPUTimeDelegateHandle   = FTSTicker::GetCoreTicker().AddTicker( GetCPUTimeDelegate );
			CPUTimeDumpDelegateHandle  = FTSTicker::GetCoreTicker().AddTicker( CPUTimeDumpDelegate, (float)Delay );
		}
	}

protected:
	/** Retrieves the current CPU time. */
	bool OnAddCPUTime( float /*DeltaTime*/ )
	{
        QUICK_SCOPE_CYCLE_COUNTER(STAT_FCPUTimeDump_OnAddCPUTime);

		const FCPUTime CPUTime = FPlatformTime::GetCPUTime();
		CPUTimes.Add( CPUTime );
		return true;
	}

	/** Prints the average CPU time from the last n frames. */
	bool OnCPUTimeDump( float /*DeltaTime*/ )
	{
        QUICK_SCOPE_CYCLE_COUNTER(STAT_FCPUTimeDump_OnCPUTimeDump);

		FCPUTime TotalCPUTime(0.0f,0.0f);
		const int32 NumCPUTimes = CPUTimes.Num();
		for( int32 Nx = 0; Nx < NumCPUTimes; ++Nx )
		{
			TotalCPUTime += CPUTimes[Nx];
		}

		UE_LOGF(LogGenericPlatformTime, Log, "CPU Time for last %3i frames: %5.1f%% (%5.1f%%)", CPUTimes.Num(), TotalCPUTime.CPUTimePct/(float)NumCPUTimes, TotalCPUTime.CPUTimePctRelative/(float)NumCPUTimes );
		CPUTimes.Reset();

		return true;
	}

	/** Array of CPU times. */
	TArray<FCPUTime> CPUTimes;

	/** Ticker delegate for getting the current CPU usage, called once per frame. */
	FTickerDelegate GetCPUTimeDelegate;

	/** Ticker delegate for printing the average CPU usage, called every nth frame. */
	FTickerDelegate CPUTimeDumpDelegate;

	/** Handle for the added GetCPUTimeDelegate delegate. */
	FTSTicker::FDelegateHandle GetCPUTimeDelegateHandle;

	/** Handle for the added CPUTimeDumpDelegate delegate. */
	FTSTicker::FDelegateHandle CPUTimeDumpDelegateHandle;
};

static FAutoConsoleCommand CPUTimeDumpCommand
(
	TEXT("CPUTime.Dump"),
	TEXT("Usage -Delay=[NumSeconds=30]\n"
	     "If Delay==0, disables printing the CPU usage to the log\n"
	     "If Delay>0, starts printing the average CPU usage from the last n frames, clamps between 10 and 300"),
	FConsoleCommandWithArgsDelegate::CreateRaw( &FCPUTimeDump::Get(), &FCPUTimeDump::ExecuteCommand )
);
