// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenericPlatform/GenericWidePlatformString.h"

#if PLATFORM_USE_GENERIC_STRING_IMPLEMENTATION

#include "HAL/UnrealMemory.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Templates/UnrealTemplate.h"

#if WITH_LOW_LEVEL_TESTS
#include "TestHarness.h"
#include <catch2/generators/catch_generators.hpp>
#endif

DEFINE_LOG_CATEGORY_STATIC(LogStandardPlatformString, Log, All);

WIDECHAR* FGenericWidePlatformString::Strcpy(WIDECHAR* Dest, const WIDECHAR* Src)
{
	TCHAR *BufPtr = Dest;
	
	while (*Src)
	{
		*BufPtr++ = *Src++;
	}
	
	*BufPtr = 0;
	
	return Dest;
}

WIDECHAR* FGenericWidePlatformString::Strncpy(WIDECHAR* Dest, const WIDECHAR* Src, SIZE_T MaxLen)
{
#if DO_CHECK
	if (UNLIKELY(MaxLen == 0))
	{
		ReportZeroStrncpySize();
	}
#endif

	TCHAR *BufPtr = Dest;
	
	// the spec says that strncpy should fill the buffer with zeroes
	// we break the spec by enforcing a trailing zero, so we do --MaxLen instead of MaxLen--
	bool bFillWithZero = false;
	while (--MaxLen)
	{
		if (bFillWithZero)
		{
			*BufPtr++ = 0;
		}
		else
		{
			if (*Src == 0)
			{
				bFillWithZero = true;
			}
			*BufPtr++ = *Src++;
		}
	}
	
	// always have trailing zero
	*BufPtr = 0;
	
	return Dest;
}

WIDECHAR* FGenericWidePlatformString::Strcat(WIDECHAR* Dest, const WIDECHAR* Src)
{
	if (!*Src)
	{
		return Dest;
	}
	TCHAR *NewDest = Dest + Strlen(Dest);
	do
	{
		*NewDest++ = *Src++;
	} while (*Src != 0);
	*NewDest = 0;
	return Dest;
}

WIDECHAR* FGenericWidePlatformString::Strncat(WIDECHAR* Dest, const WIDECHAR* Src, SIZE_T SrcLen)
{
	if (!SrcLen || !*Src)
	{
		return Dest;
	}
	TCHAR* NewDest = Dest + Strlen(Dest);
	SIZE_T AppendedCount = 0;
	do
	{
		*NewDest++ = *Src++;
		++AppendedCount;
	} while (AppendedCount < SrcLen && *Src);
	*NewDest = 0;
	return Dest;
}

int32 FGenericWidePlatformString::Strtoi( const WIDECHAR* Start, WIDECHAR** End, int32 Base )
{
#if PLATFORM_TCHAR_IS_UTF8CHAR
	unimplemented();
#else
	static_assert(sizeof(TCHAR) == 2, "TCHAR is expected to be 16-bit");
#endif

	if (End == nullptr)
	{
		return Strtoi(TCHAR_TO_UTF8(Start), nullptr, Base);
	}

#if UE_BUILD_DEBUG
	// make sure we aren't using high byte characters
	// @todo this!
#endif
	
	// convert to ANSI, and remember the end to get an offset
	auto Ansi = StringCast<ANSICHAR>(Start);
	ANSICHAR* AnsiEnd;
	int32 Result = Strtoi(Ansi.Get(), &AnsiEnd, Base);
	
	// the end is the offset from
	*End = const_cast<WIDECHAR*>(Start) + (AnsiEnd - Ansi.Get());

	return Result;
}

int64 FGenericWidePlatformString::Strtoi64( const WIDECHAR* Start, WIDECHAR** End, int32 Base )
{
#if PLATFORM_TCHAR_IS_UTF8CHAR
	unimplemented();
#else
	static_assert(sizeof(TCHAR) == 2, "TCHAR is expected to be 16-bit");
#endif

	if (End == nullptr)
	{
		return Strtoi64(TCHAR_TO_UTF8(Start), nullptr, Base);
	}

#if UE_BUILD_DEBUG
	// make sure we aren't using high byte characters
	// @todo this!
#endif
	
	// convert to ANSI, and remember the end to get an offset
	auto Ansi = StringCast<ANSICHAR>(Start);
	ANSICHAR* AnsiEnd;
	int64 Result = Strtoi64(Ansi.Get(), &AnsiEnd, Base);
	
	// the end is the offset from
	*End = const_cast<WIDECHAR*>(Start) + (AnsiEnd - Ansi.Get());

	return Result;
}

uint64 FGenericWidePlatformString::Strtoui64( const WIDECHAR* Start, WIDECHAR** End, int32 Base )
{
	if (End == nullptr)
	{
		return Strtoui64(TCHAR_TO_UTF8(Start), nullptr, Base);
	}

#if UE_BUILD_DEBUG
	// make sure we aren't using high byte characters
	// @todo this!
#endif
	
	// convert to ANSI, and remember the end to get an offset
	auto Ansi = StringCast<ANSICHAR>(Start);
	ANSICHAR* AnsiEnd;
	uint64 Result = Strtoui64(Ansi.Get(), &AnsiEnd, Base);
	
	// the end is the offset from
	*End = const_cast<WIDECHAR*>(Start) + (AnsiEnd - Ansi.Get());

	return Result;
}

WIDECHAR* FGenericWidePlatformString::Strtok(WIDECHAR* StrToken, const WIDECHAR* Delim, WIDECHAR** Context)
{
	check(Context);
	check(Delim);

	WIDECHAR* SearchString = StrToken;
	if (!SearchString)
	{
		check(*Context);
		SearchString = *Context;
	}

	WIDECHAR* TokenStart = SearchString;
	while (*TokenStart && Strchr(Delim, *TokenStart))
	{
		++TokenStart;
	}

	if (*TokenStart == 0)
	{
		return nullptr;
	}

	WIDECHAR* TokenEnd = TokenStart;
	while (*TokenEnd && !Strchr(Delim, *TokenEnd))
	{
		++TokenEnd;
	}

	*TokenEnd = 0;
	*Context = TokenEnd + 1;

	return TokenStart;
}

float FGenericWidePlatformString::Atof(const WIDECHAR* String)
{
	return Atof(TCHAR_TO_UTF8(String));
}

double FGenericWidePlatformString::Atod(const WIDECHAR* String)
{
	return Atod(TCHAR_TO_UTF8(String));
}


#if PLATFORM_ANDROID
// This is a full copy of iswspace function from Android sources
// For some reason function from libc does not work correctly for some korean characters like: 0xBE0C
int iswspace(wint_t wc)
{
	static const wchar_t spaces[] = {
		' ', '\t', '\n', '\r', 11, 12,  0x0085,
		0x2000, 0x2001, 0x2002, 0x2003, 0x2004, 0x2005,
		0x2006, 0x2008, 0x2009, 0x200a,
		0x2028, 0x2029, 0x205f, 0x3000, 0
	};
	return wc && wcschr(spaces, wc);
}
#endif

#endif // PLATFORM_USE_GENERIC_STRING_IMPLEMENTATION
