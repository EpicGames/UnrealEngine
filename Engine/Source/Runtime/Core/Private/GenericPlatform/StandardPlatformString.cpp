// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenericPlatform/StandardPlatformString.h"

// only IOS and Mac were using this file before, but that has all moved over to the FullReplacementPlatformString
#if !PLATFORM_USE_SYSTEM_VSWPRINTF && !PLATFORM_TCHAR_IS_CHAR16

#include "GenericPlatform/StandardPlatformString.h"
#include "HAL/UnrealMemory.h"
#include "Templates/UnrealTemplate.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"

#endif // !PLATFORM_USE_SYSTEM_VSWPRINTF