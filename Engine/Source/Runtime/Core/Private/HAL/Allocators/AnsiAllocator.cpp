// Copyright Epic Games, Inc. All Rights Reserved.

#include "HAL/Allocators/AnsiAllocator.h"
#include "CoreGlobals.h"

FORCENOINLINE void UE::Core::Private::OnInvalidAnsiAllocatorNum(int32 NewNum, SIZE_T NumBytesPerElement)
{
	UE_LOG(LogCore, Fatal, TEXT("Trying to resize FAnsiAllocator to an invalid size of %d with element size %" SIZE_T_FMT), NewNum, NumBytesPerElement);
	for (;;);
}

void FAnsiAllocator::ForAnyElementType::ResizeAllocation(SizeType CurrentNum, SizeType NewMax, SIZE_T NumBytesPerElement)
{
	// Avoid calling FMemory::Realloc( nullptr, 0 ) as ANSI C mandates returning a valid pointer which is not what we want.
	if (NewMax)
	{
		static_assert(sizeof(int32) <= sizeof(SIZE_T), "SIZE_T is expected to be larger than int32");

		// Check for under/overflow
		if (UNLIKELY(NewMax < 0 || NumBytesPerElement < 1 || NumBytesPerElement > (SIZE_T)MAX_int32))
		{
			UE::Core::Private::OnInvalidAnsiAllocatorNum(NewMax, NumBytesPerElement);
		}

		void* NewRealloc = ::realloc(Data, NewMax*NumBytesPerElement);
		Data = (FScriptContainerElement*)NewRealloc;
	}
	else
	{
		::free(Data);
		Data = nullptr;
	}
}
