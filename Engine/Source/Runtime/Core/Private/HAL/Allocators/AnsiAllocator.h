// Copyright Epic Games, Inc. All Rights Reserved.

/*=====================================================================================================
	AnsiAllocator.h: helper allocator that allocates directly from standard library allocation functions
=======================================================================================================*/

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "Containers/ContainerAllocationPolicies.h"

namespace UE::Core::Private
{
	[[noreturn]] CORE_API void OnInvalidAnsiAllocatorNum(int32 NewNum, SIZE_T NumBytesPerElement);
}

/** Allocator that allocates memory using standard library functions. */
class FAnsiAllocator
{
public:
	using SizeType = int32;

	enum { NeedsElementType = false };
	enum { RequireRangeCheck = true };

	typedef FAnsiAllocator ElementAllocator;
	typedef FAnsiAllocator BitArrayAllocator;

	class ForAnyElementType
	{
	public:
		/** Default constructor. */
		ForAnyElementType()
			: Data(nullptr)
		{}

		/**
		* Moves the state of another allocator into this one.
		* Assumes that the allocator is currently empty, i.e. memory may be allocated but any existing elements have already been destructed (if necessary).
		* @param Other - The allocator to move the state from.  This allocator should be left in a valid empty state.
		*/
		FORCEINLINE void MoveToEmpty(ForAnyElementType& Other)
		{
			check(this != &Other);

			if (Data)
			{
				::free(Data);
			}

			Data = Other.Data;
			Other.Data = nullptr;
		}

		/** Destructor. */
		FORCEINLINE ~ForAnyElementType()
		{
			if (Data)
			{
				::free(Data);
			}
		}

		// FContainerAllocatorInterface
		FORCEINLINE FScriptContainerElement* GetAllocation() const
		{
			return Data;
		}
		CORE_API void ResizeAllocation(SizeType CurrentNum, SizeType NewMax, SIZE_T NumBytesPerElement);
		SizeType CalculateSlackReserve(SizeType NewMax, SIZE_T NumBytesPerElement) const
		{
			return DefaultCalculateSlackReserve(NewMax, NumBytesPerElement, false);
		}
		SizeType CalculateSlackShrink(SizeType NewMax, SizeType CurrentMax, SIZE_T NumBytesPerElement) const
		{
			return DefaultCalculateSlackShrink(NewMax, CurrentMax, NumBytesPerElement, false);
		}
		SizeType CalculateSlackGrow(SizeType NewMax, SizeType CurrentMax, SIZE_T NumBytesPerElement) const
		{
			return DefaultCalculateSlackGrow(NewMax, CurrentMax, NumBytesPerElement, false);
		}

		SIZE_T GetAllocatedSize(SizeType CurrentMax, SIZE_T NumBytesPerElement) const
		{
			return CurrentMax * NumBytesPerElement;
		}

		bool HasAllocation() const
		{
			return !!Data;
		}

		SizeType GetInitialCapacity() const
		{
			return 0;
		}

	private:
		ForAnyElementType(const ForAnyElementType&);
		ForAnyElementType& operator=(const ForAnyElementType&);

		/** A pointer to the container's elements. */
		FScriptContainerElement* Data;
	};

	template<typename ElementType>
	class ForElementType : public ForAnyElementType
	{
	public:

		/** Default constructor. */
		ForElementType()
		{}

		FORCEINLINE ElementType* GetAllocation() const
		{
			return (ElementType*)ForAnyElementType::GetAllocation();
		}
	};
};

template <>
struct TAllocatorTraits<FAnsiAllocator> : TAllocatorTraitsBase<FAnsiAllocator>
{
	enum { IsZeroConstruct = true };
};

/** ANSI allocator that can be used with a TSet. */
class FAnsiSetAllocator : public TSetAllocator<FAnsiAllocator, TInlineAllocator<1, FAnsiAllocator>>
{	
};
