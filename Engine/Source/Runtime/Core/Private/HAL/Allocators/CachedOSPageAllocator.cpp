// Copyright Epic Games, Inc. All Rights Reserved.

#include "HAL/Allocators/CachedOSPageAllocator.h"
#include "CoreGlobals.h"
#include "HAL/LowLevelMemTracker.h"
#include "HAL/UnrealMemory.h"
#include "Logging/LogMacros.h"
#include "Misc/ScopeLock.h"


void* FCachedOSPageAllocator::AllocateImpl(SIZE_T Size, uint32 CachedByteLimit, FFreePageBlock* First, FFreePageBlock* Last, uint32& FreedPageBlocksNum, SIZE_T& CachedTotal, UE::FPlatformRecursiveMutex* Mutex)
{
	if (!IsOSAllocation(Size, CachedByteLimit))
	{
		if (First != Last)
		{
			FFreePageBlock* Found = nullptr;

			for (FFreePageBlock* Block = First; Block != Last; ++Block)
			{
				// look for exact matches first, these are aligned to the page size, so it should be quite common to hit these on small pages sizes
				if (Block->ByteSize == Size)
				{
					Found = Block;
					break;
				}
			}

			/* This is dubious - it results in larger block than Size bytes being returned, with no way for the client code to know the proper size
			if (!Found)
			{
				SIZE_T SizeTimes4 = Size * 4;

				for (FFreePageBlock* Block = First; Block != Last; ++Block)
				{
					// is it possible (and worth i.e. <25% overhead) to use this block
					if (Block->ByteSize >= Size && Block->ByteSize * 3 <= SizeTimes4)
					{
						Found = Block;
						break;
					}
				}
			}
			*/

			if (Found)
			{
				void* Result = Found->Ptr;
				UE_CLOGF(!Result, LogMemory, Fatal, "OS memory allocation cache has been corrupted!");
				CachedTotal -= Found->ByteSize;
				if (Found + 1 != Last)
				{
					FMemory::Memmove(Found, Found + 1, sizeof(FFreePageBlock) * ((Last - Found) - 1));
				}
				--FreedPageBlocksNum;
				return Result;
			}

			{
#if UE_ALLOW_OSMEMORYLOCKFREE
				UE::TScopeUnlock ScopeUnlock(Mutex);
#endif
				LLM_PLATFORM_SCOPE(ELLMTag::FMalloc);
				void* Ptr = FPlatformMemory::BinnedAllocFromOS(Size);
				if (Ptr)
				{
					return Ptr;
				}
			}

			// Are we holding on to much mem? Release it all.
			for (FFreePageBlock* Block = First; Block != Last; ++Block)
			{
				FPlatformMemory::BinnedFreeToOS(Block->Ptr, Block->ByteSize);
				Block->Ptr = nullptr;
				Block->ByteSize = 0;
			}
			FreedPageBlocksNum = 0;
			CachedTotal = 0;
		}
	}

	void* Ret = nullptr;
	{
#if UE_ALLOW_OSMEMORYLOCKFREE
		UE::TScopeUnlock ScopeUnlock(Mutex);
#endif
		LLM_PLATFORM_SCOPE(ELLMTag::FMalloc);
		Ret = FPlatformMemory::BinnedAllocFromOS(Size);
	}
	return Ret;
}

void FCachedOSPageAllocator::FreeImpl(void* Ptr, SIZE_T Size, uint32 NumCacheBlocks, uint32 CachedByteLimit, FFreePageBlock* First, uint32& FreedPageBlocksNum, SIZE_T& CachedTotal, UE::FPlatformRecursiveMutex* Mutex, bool ThreadIsTimeCritical)
{
	if (IsOSAllocation(Size, CachedByteLimit))
	{
#if UE_ALLOW_OSMEMORYLOCKFREE
		UE::TScopeUnlock ScopeUnlock(Mutex);
#endif
		FPlatformMemory::BinnedFreeToOS(Ptr, Size);
		return;
	}

	while (FreedPageBlocksNum && (FreedPageBlocksNum >= NumCacheBlocks || CachedTotal + Size > CachedByteLimit))
	{
		//Remove the oldest one
		void* FreePtr = First->Ptr;
		SIZE_T FreeSize = First->ByteSize;
		CachedTotal -= FreeSize;
		FreedPageBlocksNum--;
		if (FreedPageBlocksNum)
		{
			FMemory::Memmove(First, First + 1, sizeof(FFreePageBlock) * FreedPageBlocksNum);
		}
		{
#if UE_ALLOW_OSMEMORYLOCKFREE
			UE::TScopeUnlock ScopeUnlock(Mutex);
#endif
			FPlatformMemory::BinnedFreeToOS(FreePtr, FreeSize);
		}
	}

	First[FreedPageBlocksNum].Ptr      = Ptr;
	First[FreedPageBlocksNum].ByteSize = Size;

	CachedTotal += Size;
	++FreedPageBlocksNum;
}

void FCachedOSPageAllocator::FreeAllImpl(FFreePageBlock* First, uint32& FreedPageBlocksNum, SIZE_T& CachedTotal, UE::FPlatformRecursiveMutex* Mutex)
{
	while (FreedPageBlocksNum)
	{
		//Remove the oldest one
		void* FreePtr = First->Ptr;
		SIZE_T FreeSize = First->ByteSize;
		CachedTotal -= FreeSize;
		FreedPageBlocksNum--;
		if (FreedPageBlocksNum)
		{
			FMemory::Memmove(First, First + 1, sizeof(FFreePageBlock)* FreedPageBlocksNum);
		}
		FPlatformMemory::BinnedFreeToOS(FreePtr, FreeSize);
	}
}
