// Copyright Epic Games, Inc. All Rights Reserved.

#include "HAL/Allocators/CachedOSVeryLargePageAllocator.h"
#include "Async/UniqueLock.h"
#include "HAL/IConsoleManager.h"
#include "HAL/UnrealMemory.h"
#include "Logging/LogMacros.h"
#include "CoreGlobals.h"
#include "HAL/LowLevelMemTracker.h"
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformTime.h"
#include "ProfilingDebugging/CsvProfiler.h"

#if UE_USE_VERYLARGEPAGEALLOCATOR

CORE_API bool GEnableVeryLargePageAllocator = true;

static_assert(int32(FMemory::AllocationHints::Max) == 3); // ensure FMemory::AllocationHints has 3 types of hint so GMaxEmptyBackStoreCount has needed values

static bool GPreAllocatePools = true;
static FAutoConsoleVariableRef CVarPreAllocatePools(
	TEXT("VeryLargePageAllocator.PreAllocatePools"),
	GPreAllocatePools,
	TEXT("Having pages preallocated and cached during the life of the title help to avoid defragmentation of physical memory.\n")
	TEXT("Preallocation may be disabled when system reaches OOM (see VeryLargePageAllocator.DisablePageCachingOnOOM)"));

static bool GDisablePageCachingOnOOM = false;
static FAutoConsoleVariableRef CVarDisablePageCachingOnOOM(
	TEXT("VeryLargePageAllocator.DisablePageCachingOnOOM"),
	GDisablePageCachingOnOOM,
	TEXT("If enabled, permanently disable page caching when a OOM happens and all unused pages have been freed (so new allocated pages gets cached again)\n")
	TEXT("This can lead to unpredictable performance on some platforms."));

static int32 GMaxEmptyBackStoreCount[FMemory::AllocationHints::Max] = {
	0,	// FMemory::AllocationHints::Default
	0,	// FMemory::AllocationHints::Temporary
	0	// FMemory::AllocationHints::SmallPool
};
static FAutoConsoleVariableRef CVarMaxEmptyBackstoreDefault(
	TEXT("VeryLargePageAllocator.MaxEmptyBackstoreDefault"),
	GMaxEmptyBackStoreCount[FMemory::AllocationHints::Default],
	TEXT("Number of free pages (2MB each) to cache (not decommitted) for allocation hint DEFAULT"));

static FAutoConsoleVariableRef CVarMaxEmptyBackstoreSmallPool(
	TEXT("VeryLargePageAllocator.MaxEmptyBackstoreSmallPool"),
	GMaxEmptyBackStoreCount[FMemory::AllocationHints::SmallPool],
	TEXT("Number of free pages (2MB each) to cache (not decommitted) for allocation hint SMALL POOL"));


static int32 GMaxCommittedPageCount[FMemory::AllocationHints::Max] = {
	0,	// FMemory::AllocationHints::Default
	0,	// FMemory::AllocationHints::Temporary
	0	// FMemory::AllocationHints::SmallPool
};

static FAutoConsoleVariableRef CVarMaxPageCountDefault(
	TEXT("VeryLargePageAllocator.MaxCommittedPageCountDefault"),
	GMaxCommittedPageCount[FMemory::AllocationHints::Default],
	TEXT("Maximum number of pages (2MB each) to use for allocation hint DEFAULT")
	TEXT("Adjustable at runtime. If we run out of pages we'll fall back to the standard allocator"));

static FAutoConsoleVariableRef CVarMaxPageCountSmallPool(
	TEXT("VeryLargePageAllocator.MaxCommittedPageCountSmallPool"),
	GMaxCommittedPageCount[FMemory::AllocationHints::SmallPool],
	TEXT("Maximum number of pages (2MB each) to use for allocation hint SMALL POOL\n")
	TEXT("Adjustable at runtime. If we run out of pages we'll fall back to the standard allocator"));


#if CSV_PROFILER_STATS
CSV_DECLARE_CATEGORY_MODULE_EXTERN(CORE_API, FMemory);

static std::atomic<int32> GLargePageAllocatorCommitCount = 0;
static std::atomic<int32> GLargePageAllocatorDecommitCount = 0;
#endif

void FCachedOSVeryLargePageAllocator::Init()
{
	Block = FPlatformMemory::FPlatformVirtualMemoryBlock::AllocateVirtual(AddressSpaceToReserve);
	AddressSpaceReserved = (uintptr_t)Block.GetVirtualPointer();
	AddressSpaceReservedEnd = AddressSpaceReserved + AddressSpaceToReserve;
	AddressSpaceReservedEndSmallPool = AddressSpaceReserved + (AddressSpaceToReserve / 2);

	for (int i = 0; i < FMemory::AllocationHints::Max; i++)
	{
		FreeLargePagesHead[i] = nullptr;
		UsedLargePagesWithSpaceHead[i] = nullptr;
		UsedLargePagesHead[i] = nullptr;
		EmptyButAvailableLargePagesHead[i] = nullptr;
		EmptyBackStoreCount[i] = 0;
		CommittedLargePagesCount[i] = 0;
	}
	for (int i = 0; i < NumberOfLargePages / 2; i++)
	{
		LargePagesArray[i].Init((void*)((uintptr_t)AddressSpaceReserved + (i * SizeOfLargePage)));
		LargePagesArray[i].LinkHead(FreeLargePagesHead[FMemory::AllocationHints::SmallPool]);
	}

	for (int i = NumberOfLargePages / 2; i < NumberOfLargePages; i++)
	{
		LargePagesArray[i].Init((void*)((uintptr_t)AddressSpaceReserved + (i * SizeOfLargePage)));
		LargePagesArray[i].LinkHead(FreeLargePagesHead[FMemory::AllocationHints::Default]);
	}

	if (!GEnableVeryLargePageAllocator)
	{
		bEnabled = false;
	}
}

void FCachedOSVeryLargePageAllocator::Refresh()
{
	if (!bEnabled)
	{
		return;
	}
	
	// Shrink Empty back store
	for (int i = 0; i < FMemory::AllocationHints::Max; i++)
	{
		FMemory::AllocationHints AllocationHint = FMemory::AllocationHints(i);
		ShrinkEmptyBackStore(0, AllocationHint);
	}

	// Preallocate
	for (int32 i = 0; i < FMemory::AllocationHints::Max; i++)
	{
		FMemory::AllocationHints AllocationHint = FMemory::AllocationHints(i);

		int32 LargePageCount = CommittedLargePagesCount[AllocationHint] + EmptyBackStoreCount[AllocationHint];

		if (LargePageCount < GMaxEmptyBackStoreCount[AllocationHint] && GPreAllocatePools)
		{
			// Preallocate large pages
			for (; LargePageCount < GMaxEmptyBackStoreCount[AllocationHint]; LargePageCount++)
			{
				bool bCommitFailure = false;
				FLargePage* LargePage = AllocNewLargePage(AllocationHint, nullptr, bCommitFailure);
				if (!LargePage)
				{
					if (bCommitFailure && GDisablePageCachingOnOOM)
					{
						// If we failed to preallocate due to a commit failure then disable preallocation in future
						GPreAllocatePools = false;
					}
					break;
				}
				LargePage->LinkHead(EmptyButAvailableLargePagesHead[AllocationHint]);
				EmptyBackStoreCount[AllocationHint] += 1;
				ImmediatelyFreeable += SizeOfLargePage;
			}
		}
	}
}

void* FCachedOSVeryLargePageAllocator::Allocate(SIZE_T Size, uint32 AllocationHint, UE::FPlatformRecursiveMutex* Mutex)
{
	Size = Align(Size, 4096);

	void* ret = nullptr;

	if (bEnabled && Size == SizeOfSubPage)
	{
		FLargePage* LargePage = GetOrAllocLargePage(AllocationHint, Mutex);

		if (LargePage)
		{
			ret = LargePage->Allocate();
			if (ret)
			{
				// If the page is full then move it to the full list
				if (LargePage->NumberOfFreeSubPages == 0) 
				{
					LargePage->Unlink();
					LargePage->LinkHead(UsedLargePagesHead[AllocationHint]);
				}
				CachedFree -= SizeOfSubPage;
			}
			else
			{
				if (AllocationHint == FMemory::AllocationHints::SmallPool)
				{
					UE_CLOGF(!ret, LogMemory, Fatal, "The FCachedOSVeryLargePageAllocator has run out of address space for SmallPool allocations, increase UE_VERYLARGEPAGEALLOCATOR_RESERVED_SIZE_IN_GB for your platform!");
				}
			}
		}
	}

	if (ret == nullptr)
	{
		ret = CachedOSPageAllocator.Allocate(Size, AllocationHint, Mutex);
	}
	return ret;
}


FCachedOSVeryLargePageAllocator::FLargePage* FCachedOSVeryLargePageAllocator::GetOrAllocLargePage(uint32 AllocationHint, UE::FPlatformRecursiveMutex* Mutex)
{
	// Use an existing page with space if one is available
	FLargePage* LargePage = UsedLargePagesWithSpaceHead[AllocationHint];
	if (LargePage)
	{
		return LargePage;
	}

	// Attempt to allocate an empty already-committed page
	LargePage = EmptyButAvailableLargePagesHead[AllocationHint];
	if (LargePage)
	{
		LargePage->AllocationHint = AllocationHint;
		LargePage->Unlink();
		EmptyBackStoreCount[AllocationHint] -= 1;
		ImmediatelyFreeable -= SizeOfLargePage;
	}
	else
	{
		// If all else fails, allocate and commit a new page
		bool bCommitFailureUnused;
		LargePage = AllocNewLargePage(AllocationHint, Mutex, bCommitFailureUnused);
	}

	// Move the page to the active list
	if (LargePage)
	{
		LargePage->LinkHead(UsedLargePagesWithSpaceHead[AllocationHint]);
	}
	return LargePage;
}

FCachedOSVeryLargePageAllocator::FLargePage* FCachedOSVeryLargePageAllocator::AllocNewLargePage(uint32 AllocationHint, UE::FPlatformRecursiveMutex* Mutex, bool& bOutCommitFailure)
{
	// If there's a limit for this allocation type then check first
	bOutCommitFailure = false;
	if (GMaxCommittedPageCount[AllocationHint] > 0 && CommittedLargePagesCount[AllocationHint] >= GMaxCommittedPageCount[AllocationHint])
	{
		return nullptr;
	}

	FLargePage* LargePage = FreeLargePagesHead[AllocationHint];
	if (LargePage != nullptr)
	{
		LargePage->AllocationHint = AllocationHint;
		LargePage->Unlink();
		{
#if UE_ALLOW_OSMEMORYLOCKFREE
			UE::TScopeUnlock ScopeUnlock(Mutex);
#endif
			LLM_PLATFORM_SCOPE(ELLMTag::FMalloc);
			if (!Block.Commit(LargePage->BaseAddress - AddressSpaceReserved, SizeOfLargePage, false))
			{
#if UE_ALLOW_OSMEMORYLOCKFREE
				if (Mutex != nullptr)
				{
					UE::TUniqueLock Lock(*Mutex);
					LargePage->LinkHead(FreeLargePagesHead[AllocationHint]);
				}
				else
#endif
				{
					LargePage->LinkHead(FreeLargePagesHead[AllocationHint]);
				}
				bOutCommitFailure = true;
				return nullptr;
			}
			LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelAlloc(ELLMTracker::Platform, (void*)LargePage->BaseAddress, SizeOfLargePage));
			// A new large page has been created. Add it to CachedFree counter
			CachedFree += SizeOfLargePage;
			CommittedLargePagesCount[AllocationHint] += 1;
#if CSV_PROFILER_STATS
			GLargePageAllocatorCommitCount.fetch_add(1, std::memory_order_relaxed);
#endif
		}
	}
	return LargePage;
}

void FCachedOSVeryLargePageAllocator::Free(void* Ptr, SIZE_T Size, UE::FPlatformRecursiveMutex* Mutex, bool ThreadIsTimeCritical)
{
	Size = Align(Size, 4096);
	uint64 Index = ((uintptr_t)Ptr - (uintptr_t)AddressSpaceReserved) / SizeOfLargePage;
	if (Index < (NumberOfLargePages))
	{
		FLargePage* LargePage = &LargePagesArray[Index];

		LargePage->Free(Ptr);
		CachedFree += SizeOfSubPage;

		if (LargePage->NumberOfFreeSubPages == NumberOfSubPagesPerLargePage)
		{
			// totally free
			LargePage->Unlink();

			// move it to EmptyButAvailableLargePagesHead if that pool of backstore is not full yet
			if (EmptyBackStoreCount[LargePage->AllocationHint] < GMaxEmptyBackStoreCount[LargePage->AllocationHint])
			{
				LargePage->LinkHead(EmptyButAvailableLargePagesHead[LargePage->AllocationHint]);
				EmptyBackStoreCount[LargePage->AllocationHint] += 1;
				ImmediatelyFreeable += SizeOfLargePage;
			}
			else
			{
				// need to move which list we are in and remove the backing store
				{
#if UE_ALLOW_OSMEMORYLOCKFREE
					UE::TScopeUnlock ScopeUnlock(Mutex);
#endif
					Block.Decommit(LargePage->BaseAddress - AddressSpaceReserved, SizeOfLargePage);
					LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelFree(ELLMTracker::Platform, (void*)LargePage->BaseAddress));
				}

				CommittedLargePagesCount[LargePage->AllocationHint] -= 1;
#if CSV_PROFILER_STATS
				GLargePageAllocatorDecommitCount.fetch_add(1, std::memory_order_relaxed);
#endif

				LargePage->LinkHead(FreeLargePagesHead[LargePage->AllocationHint]);
				CachedFree -= SizeOfLargePage;
			}
		}
		else if (LargePage->NumberOfFreeSubPages == 1)
		{
			LargePage->Unlink();
			// Sort on address
			FLargePage* InsertPoint = UsedLargePagesWithSpaceHead[LargePage->AllocationHint];
			while (InsertPoint != nullptr)
			{
				if (LargePage->BaseAddress < InsertPoint->BaseAddress)
				{
					break;
				}
				InsertPoint = InsertPoint->Next();
			}
			if (InsertPoint == nullptr || InsertPoint == UsedLargePagesWithSpaceHead[LargePage->AllocationHint])
			{
				LargePage->LinkHead(UsedLargePagesWithSpaceHead[LargePage->AllocationHint]);
			}
			else
			{
				LargePage->LinkBefore(InsertPoint);
			}
		}
	}
	else
	{
		CachedOSPageAllocator.Free(Ptr, Size, Mutex, ThreadIsTimeCritical);
	}
}

void FCachedOSVeryLargePageAllocator::ShrinkEmptyBackStore(int32 NewEmptyBackStoreSize, FMemory::AllocationHints AllocationHint)
{
	while (EmptyBackStoreCount[AllocationHint] > NewEmptyBackStoreSize)
	{
		FLargePage* LargePage = EmptyButAvailableLargePagesHead[AllocationHint];

		if (LargePage == nullptr)
		{
			break;
		}
		LargePage->Unlink();
		Block.Decommit(LargePage->BaseAddress - AddressSpaceReserved, SizeOfLargePage);
		LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelFree(ELLMTracker::Platform, (void*)LargePage->BaseAddress));
		LargePage->LinkHead(FreeLargePagesHead[LargePage->AllocationHint]);
		CachedFree -= SizeOfLargePage;
		ImmediatelyFreeable -= SizeOfLargePage;
		EmptyBackStoreCount[AllocationHint] -= 1;
		CommittedLargePagesCount[LargePage->AllocationHint] -= 1;

#if CSV_PROFILER_STATS
		GLargePageAllocatorDecommitCount.fetch_add(1, std::memory_order_relaxed);
#endif
	}
}


void FCachedOSVeryLargePageAllocator::FreeAll(UE::FPlatformRecursiveMutex* Mutex)
{
	for (int i = 0; i < FMemory::AllocationHints::Max; i++)
	{
		FMemory::AllocationHints AllocationHint = FMemory::AllocationHints(i);
		ShrinkEmptyBackStore(0, AllocationHint);
	}

	if (GDisablePageCachingOnOOM)
	{
		// Stop preallocating system since allocator reached a OOM
		GPreAllocatePools = false;
	}

	// Free empty cached pages of CachedOSPageAllocator
	CachedOSPageAllocator.FreeAll(Mutex);
}

void FCachedOSVeryLargePageAllocator::UpdateStats()
{
#if CSV_PROFILER_STATS
	CSV_CUSTOM_STAT(FMemory, LargeAllocatorCommitCount, GLargePageAllocatorCommitCount.load(std::memory_order_relaxed), ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT(FMemory, LargeAllocatorDecommitCount, GLargePageAllocatorDecommitCount.load(std::memory_order_relaxed), ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT(FMemory, LargeAllocatorBackingStoreCountSmall, EmptyBackStoreCount[FMemory::AllocationHints::SmallPool], ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT(FMemory, LargeAllocatorBackingStoreCountDefault, EmptyBackStoreCount[FMemory::AllocationHints::Default], ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT(FMemory, LargeAllocatorPageCountSmall, CommittedLargePagesCount[FMemory::AllocationHints::SmallPool], ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT(FMemory, LargeAllocatorPageCountDefault, CommittedLargePagesCount[FMemory::AllocationHints::Default], ECsvCustomStatOp::Set);

	GLargePageAllocatorCommitCount.store(0, std::memory_order_relaxed);
	GLargePageAllocatorDecommitCount.store(0, std::memory_order_relaxed);
#endif
}
#endif
