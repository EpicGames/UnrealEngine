// Copyright Epic Games, Inc. All Rights Reserved.

#include "HAL/Allocators/PooledVirtualMemoryAllocator.h"
#include "Async/UniqueLock.h"
#include "HAL/UnrealMemory.h"
#include "Logging/LogMacros.h"
#include "CoreGlobals.h"
#include "HAL/LowLevelMemTracker.h"
#include "Misc/AssertionMacros.h"
#include "Misc/ScopeLock.h"
#include "Templates/AlignmentTemplates.h"
#include "GenericPlatform/OSAllocationPool.h"

#if PLATFORM_HAS_FPlatformVirtualMemoryBlock

using T64KBAlignedPool = TMemoryPool<65536>;

#ifndef UE_VMA_POOL_SCALE
	#if !UE_EDITOR
		// all non-editor targets have been long running with -vmapoolscale=1.0
		#define UE_VMA_POOL_SCALE 1.0f
	#else
		#define UE_VMA_POOL_SCALE 1.4f
	#endif
#endif

/** Scale parameter used when growing the pools on allocation (and scaling them back), configurable from the commandline */
float GVMAPoolScale = UE_VMA_POOL_SCALE;

struct FPoolDescriptor : public FPooledVirtualMemoryAllocator::FPoolDescriptorBase
{
	/** Pool itself */
	T64KBAlignedPool* Pool;
};

FPooledVirtualMemoryAllocator::FPooledVirtualMemoryAllocator()
{
	// The first time a pool for any allocation size class is created, it be close to this size
	// Note: how close depends on DecideOnTheNextPoolSize behavior, which will first grow it
	const int32 kInitialPoolSize = 8 * 1024 * 1024;

	for (int32 IdxClass = 0; IdxClass < Limits::NumAllocationSizeClasses; ++IdxClass)
	{
		const int32 SizeOfAllocationInPool = (IdxClass + 1) * 65536;
		NextPoolSize[IdxClass] = FMath::Max(2, kInitialPoolSize / SizeOfAllocationInPool);

		ClassesListHeads[IdxClass] = nullptr;
	}
}

void* FPooledVirtualMemoryAllocator::Allocate(SIZE_T Size, uint32 AllocationHint, UE::FPlatformRecursiveMutex* Mutex)
{
	if (Size > Limits::MaxAllocationSizeToPool)
	{
		if (OsAllocatorCache.IsOSAllocation(Size))
		{
			// do not report to LLM here, the platform functions will do that
#if UE_ALLOW_OSMEMORYLOCKFREE
			UE::TScopeUnlock ScopeUnlock(Mutex);
#endif
			return FPlatformMemory::BinnedAllocFromOS(Size);
		}
		else
		{
			// do not report to LLM here, the platform functions will do that
			UE::TUniqueLock Lock(OsAllocatorCacheLock);
			void* Ptr = OsAllocatorCache.Allocate(Size);
			return Ptr;
		}
	}
	else
	{
		int32 SizeClass = GetAllocationSizeClass(Size);

		LLM_PLATFORM_SCOPE(ELLMTag::FMalloc);

		// [RCL] TODO: find a way to convert to lock-free
		UE::TUniqueLock Lock(ClassesLocks[SizeClass]);

		// follow the list until we can allocate
		for(FPoolDescriptorBase* BaseDesc = ClassesListHeads[SizeClass]; BaseDesc; BaseDesc = BaseDesc->Next)
		{
			// using reference to avoid static_cast cost of checking the pointer with null, we know it's not null
			FPoolDescriptor& Desc = static_cast<FPoolDescriptor&>(*BaseDesc);

			if (void* Ptr = Desc.Pool->Allocate(Size))
			{
				// LLM wants to be informed of the allocations of physical RAM, this is the closest we can get.
				LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelAlloc(ELLMTracker::Platform, Ptr, Size));
				return Ptr;
			}
		}

		DecideOnTheNextPoolSize(SizeClass, true);

		// we exhausted existing pools, allocate a new one
		FPoolDescriptorBase* NewPool = CreatePool(CalculateAllocationSizeFromClass(SizeClass), NextPoolSize[SizeClass]);
		if (UNLIKELY(NewPool == nullptr))
		{
			FPlatformMemory::OnOutOfMemory(Size, 65536);
			// unreachable
		}

		// add to the list, making it the new head
		// the reasoning here is that each new pool will have a larger size,
		// so it's better to have them sorted by size descending
		NewPool->Next = ClassesListHeads[SizeClass];
		ClassesListHeads[SizeClass] = NewPool;

		FPoolDescriptor& Desc = static_cast<FPoolDescriptor&>(*NewPool);
		// should not fail at this point
		void* Ptr = Desc.Pool->Allocate(Size);

		// LLM wants to be informed of the allocations of physical RAM, this is the closest we can get.
		LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelAlloc(ELLMTracker::Platform, Ptr, Size));
		return Ptr;
	}
};

void FPooledVirtualMemoryAllocator::Free(void* Ptr, SIZE_T Size, UE::FPlatformRecursiveMutex* Mutex, bool /*ThreadIsTimeCritical = false*/)
{
	if (Size > Limits::MaxAllocationSizeToPool)
	{
		if (OsAllocatorCache.IsOSAllocation(Size))
		{
			// do not report to LLM here, the platform functions will do that
#if UE_ALLOW_OSMEMORYLOCKFREE
			UE::TScopeUnlock ScopeUnlock(Mutex);
#endif
			FPlatformMemory::BinnedFreeToOS(Ptr, Size);
		}
		else
		{
			// do not report to LLM here, the platform functions will do that
			UE::TUniqueLock Lock(OsAllocatorCacheLock);
			OsAllocatorCache.Free(Ptr, Size);
		}
	}
	else
	{
		int32 SizeClass = GetAllocationSizeClass(Size);

		// [RCL] TODO: find a way to convert to lock-free
		UE::TUniqueLock Lock(ClassesLocks[SizeClass]);

		// follow the list until we can find the pool it came from
		FPoolDescriptorBase* PrevBaseDesc = nullptr;
		for(FPoolDescriptorBase* BaseDesc = ClassesListHeads[SizeClass]; BaseDesc; PrevBaseDesc = BaseDesc, BaseDesc = BaseDesc->Next)
		{
			// using reference to avoid static_cast cost of checking the pointer with null, we know it's not null
			FPoolDescriptor& Desc = static_cast<FPoolDescriptor&>(*BaseDesc);

			if (UNLIKELY(Desc.Pool->WasAllocatedFromThisPool(Ptr, Size)))
			{
				// LLM wants to be informed of the allocations of physical RAM.
				// This is the closest we can get.
				LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelFree(ELLMTracker::Platform, Ptr));
				Desc.Pool->Free(Ptr, Size);

				// check if the pool is empty and delete if so
				// Note: could defer until Trim() is called
				if (UNLIKELY(Desc.Pool->IsEmpty()))
				{
					// unchain from the list
					if (LIKELY(PrevBaseDesc))
					{
						PrevBaseDesc->Next = Desc.Next;
					}
					else
					{
						ClassesListHeads[SizeClass] = Desc.Next;
					}

					DestroyPool(BaseDesc);
					DecideOnTheNextPoolSize(SizeClass, false);
				}
				break;
			}
		}
	}
};

void FPooledVirtualMemoryAllocator::DecideOnTheNextPoolSize(int32 SizeClass, bool bGrowing)
{
	// heuristic, attempts to scale exponentially
	if (bGrowing)
	{
		NextPoolSize[SizeClass] = static_cast<int32>(GVMAPoolScale * static_cast<float>(NextPoolSize[SizeClass]));
	}
	else
	{
		NextPoolSize[SizeClass] = FMath::Max(2, static_cast<int32>(static_cast<float>(NextPoolSize[SizeClass]) / GVMAPoolScale));
	}
}

FPooledVirtualMemoryAllocator::FPoolDescriptorBase* FPooledVirtualMemoryAllocator::CreatePool(SIZE_T AllocationSize, int32 NumPooledAllocations)
{
	// calculate total size needed from the OS
	SIZE_T TotalSize = 0;

	// We will store descriptor and the pool bookkeeping data at the head of the allocation
	// 1) the descriptor size
	SIZE_T DescriptorSize = sizeof(FPoolDescriptor);
	TotalSize += DescriptorSize;

	// 2) the book-keeping memory for the pool
	SIZE_T PoolClassSizeof = sizeof(*FPoolDescriptor::Pool);
	TotalSize += PoolClassSizeof;
	SIZE_T BookkeepingMemorySize = T64KBAlignedPool::BitmaskMemorySize(NumPooledAllocations);
	TotalSize += BookkeepingMemorySize;

	// All the above memory will be the "header", the pool memory itself will begin from there, 64KB-aligned
	SIZE_T HeaderSize = TotalSize;

	// Let's add 64KB padding so we can find a 64KB-aligned pointer after the header
	TotalSize = TotalSize + 65536;

	// now add the main memory requirements
	TotalSize += AllocationSize * static_cast<SIZE_T>(NumPooledAllocations);

	FPlatformMemory::FPlatformVirtualMemoryBlock VMBlock = FPlatformMemory::FPlatformVirtualMemoryBlock::AllocateVirtual(Align(TotalSize, FPlatformMemory::FPlatformVirtualMemoryBlock::GetVirtualSizeAlignment()));

	uint8* RawPtr = static_cast<uint8*>(VMBlock.GetVirtualPointer());
	if (RawPtr == nullptr)
	{
		return nullptr;
	}    

	// Commit the header so we can touch it
	VMBlock.Commit(0, Align(HeaderSize, FPlatformMemory::FPlatformVirtualMemoryBlock::GetCommitAlignment()));
	FPoolDescriptor* Ptr = reinterpret_cast<FPoolDescriptor*>(RawPtr);
	// store the total size to deallocate
	Ptr->VMSizeDivVirtualSizeAlignment = VMBlock.GetActualSizeInPages();

	// find different offsets
	uint8* PointerToPool = RawPtr + DescriptorSize;
	uint8* PointerToBookkeepingMemory = PointerToPool + PoolClassSizeof;
	uint8* MemoryAfterTheHeader = PointerToBookkeepingMemory + BookkeepingMemorySize;

	uint8* AlignedMemoryForThePool = Align(MemoryAfterTheHeader, 65536);
	Ptr->Pool = new (PointerToPool) T64KBAlignedPool(AllocationSize, reinterpret_cast<SIZE_T>(AlignedMemoryForThePool), NumPooledAllocations, 
		PointerToBookkeepingMemory, VMBlock);

	// mmap can return allocation that is not 64k aligned, so we had to add 64k padding after the header.
	// But we can trim excess pages after the fact.
	uint8* AlignedMemoryAfterThePool = Align(AlignedMemoryForThePool + AllocationSize * static_cast<SIZE_T>(NumPooledAllocations), FPlatformMemory::FPlatformVirtualMemoryBlock::GetVirtualSizeAlignment());
	SIZE_T PagesUsed = (AlignedMemoryAfterThePool - RawPtr) / FPlatformMemory::FPlatformVirtualMemoryBlock::GetVirtualSizeAlignment();
	SIZE_T PagesLeft = Ptr->VMSizeDivVirtualSizeAlignment - PagesUsed;
	if (PagesLeft > 0)
	{
		checkf(Ptr->VMSizeDivVirtualSizeAlignment > PagesLeft, TEXT("Arithmetic error calculating excess pages"));
		Ptr->VMSizeDivVirtualSizeAlignment -= PagesLeft;
		FPlatformMemory::FPlatformVirtualMemoryBlock(AlignedMemoryAfterThePool, (uint32)PagesLeft).FreeVirtual();
	}

	return Ptr;
}

void FPooledVirtualMemoryAllocator::DestroyPool(FPoolDescriptorBase* Pool)
{
	// we're sure it cannot be null
	checkf(Pool, TEXT("Passed a null pool descriptor pointer to FreePool()"));
	FPoolDescriptor& PoolDesc = static_cast<FPoolDescriptor &>(*Pool);

	// allocated with placement new, do not call delete
	PoolDesc.Pool->~T64KBAlignedPool();

	FPlatformMemory::FPlatformVirtualMemoryBlock VMBlock(Pool, (uint32)Pool->VMSizeDivVirtualSizeAlignment);
	VMBlock.FreeVirtual();
}

void FPooledVirtualMemoryAllocator::FreeAll(UE::FPlatformRecursiveMutex* /*Mutex = nullptr*/)
{
	UE::TUniqueLock Lock(OsAllocatorCacheLock);
	OsAllocatorCache.FreeAll();

	// Currently, there's nothing else to trim.
	// We could avoid deleting pools on Free() and instead keep them in a separate list to delete on FreeAll() (unless they're reused before that).
	// That would be a speed optimization and not a size optimization so I'm not going for this at this point, this method is speedy enough.
};

uint64 FPooledVirtualMemoryAllocator::GetCachedFreeTotal() const
{
	uint64 TotalFree = 0;

	for (int32 IdxSizeClass = 0; IdxSizeClass < Limits::NumAllocationSizeClasses; ++IdxSizeClass)
	{
		UE::TUniqueLock Lock(ClassesLocks[IdxSizeClass]);

		for(FPoolDescriptorBase* BaseDesc = ClassesListHeads[IdxSizeClass]; BaseDesc; BaseDesc = BaseDesc->Next)
		{
			// using reference to avoid static_cast cost of checking the pointer with null, we know it's not null
			FPoolDescriptor& Desc = static_cast<FPoolDescriptor&>(*BaseDesc);

			// not accounting for the overhead here since we cannot make use of that "free" memory anyway
			TotalFree += Desc.Pool->GetAllocatableMemorySize();
		}
	}

	return TotalFree;
}

void FPooledVirtualMemoryAllocator::DumpAllocatorStats(FOutputDevice& Ar)
{
	for (int32 IdxSizeClass = 0; IdxSizeClass < Limits::NumAllocationSizeClasses; ++IdxSizeClass)
	{
		UE::TUniqueLock Lock(ClassesLocks[IdxSizeClass]);
		SIZE_T AllocationSizeForClass = CalculateAllocationSizeFromClass(IdxSizeClass);
		Ar.Logf(TEXT("PooledVirtualMemoryAllocator Index: %d, SizeClass: %.2fKB"),
			IdxSizeClass,
			(double)AllocationSizeForClass / 1024.0);
		for(FPoolDescriptorBase* BaseDesc = ClassesListHeads[IdxSizeClass]; BaseDesc; BaseDesc = BaseDesc->Next)
		{
			FPoolDescriptor& Desc = static_cast<FPoolDescriptor&>(*BaseDesc);

			Ar.Logf(TEXT("Pool[%3d]: %10.2fKB allocatable, %10.2fKB overhead"),
				IdxSizeClass,
				(double)Desc.Pool->GetAllocatableMemorySize() / 1024.0,
				(double)Desc.Pool->GetOverheadSize() / 1024.0);
		}
	}

	UE::TUniqueLock Lock(OsAllocatorCacheLock);
	OsAllocatorCache.DumpAllocatorStats(Ar);
}
#endif
