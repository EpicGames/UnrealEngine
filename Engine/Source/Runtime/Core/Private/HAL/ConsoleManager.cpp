// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
ConsoleManager.cpp: console command handling
=============================================================================*/

#include "HAL/ConsoleManager.h"

#include "HAL/FileManager.h"
#include "HAL/PlatformProcess.h"
#include "Logging/MessageLog.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/ConfigContext.h"
#include "Misc/ConfigUtilities.h"
#include "Misc/CoreDelegates.h"
#include "Modules/ModuleManager.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "Stats/Stats.h"
#include "Trace/Trace.h"
#include "Trace/Trace.inl"

#include <clocale>

DEFINE_LOG_CATEGORY(LogConsoleResponse);
DEFINE_LOG_CATEGORY_STATIC(LogConsoleManager, Log, All);

#define LOCTEXT_NAMESPACE "ConsoleManager"

class FOutputDevice;
class UWorld;

static FConsoleManager& GetManager()
{
	return static_cast<FConsoleManager&>(IConsoleManager::Get());
}

namespace UE::ConsoleManager::Private
{
	// this tracks the cvars that were added dynamically with a tag (via plugin or similar)
	// we use this structure to unset the cvars and update the value when the plugin unloads
	TMap<FName, TSet<IConsoleVariable*>*> TaggedCVars;

	/**
	 * Setup a locale for a given scope, the previous locale is restored on scope end.
	 */
	struct FConsoleManagerLocaleScope
	{
		// Set a custom locale within this object scope
		FConsoleManagerLocaleScope()
		{
			if (AutoRTFM::IsClosed())
			{
				// We only support locale setting if the scope is on the stack. This
				// is just a defensive block against any new usecase potentially doing
				// a heap allocation wrongly!
				ensure(AutoRTFM::IsOnCurrentTransactionStack(this));

				// If we abort while `this` is currently alive, we need to run the
				// destructor so that any locale we had temporarily disturbed would be
				// restored correctly.
				AutoRTFM::PushOnAbortHandler(this, [this]
				{
					this->~FConsoleManagerLocaleScope();
				});
			}

			AutoRTFM::Open([&]
			{
				if (const char* saved = std::setlocale(LC_NUMERIC, nullptr))
				{
					SavedLocale = TArray<char>(saved, TCString<char>::Strlen(saved) + 1);
				}
				std::setlocale(LC_NUMERIC, "C");
			});
		}

		// restore the captured Locale
		~FConsoleManagerLocaleScope()
		{
			AutoRTFM::Open([&]
			{
				std::setlocale(LC_NUMERIC, SavedLocale.GetData());
			});

			if (AutoRTFM::IsClosed())
			{
				// We just restored the locale to the previous value above, so we pop here
				// to ensure we don't try and do a double restore incorrectly.
				AutoRTFM::PopOnAbortHandler(this);
			}
		}

	private:
		TArray<char> SavedLocale; // Locale previously used, captured, and to be restored
	};

	template<typename T>
	void GetValueFromString(T& Value, const TCHAR* Buffer)
	{
		TTypeFromString<T>::FromString(Value, Buffer);
	};

	template<>
	void GetValueFromString<float>(float& Value, const TCHAR* Buffer)
	{
		FConsoleManagerLocaleScope LocaleScope;
		TTypeFromString<float>::FromString(Value, Buffer);
	};
}

static inline bool IsWhiteSpace(TCHAR Value) { return Value == TCHAR(' '); }

// @param In must not be 0
bool IsGoodHelpString(const TCHAR* In)
{
	check(In);

	if(*In == 0)
	{
		return false;
	}

	bool bGoodEndChar = true;

	while(TCHAR c = *In++)
	{
		bGoodEndChar = true;

		if(c == L'\n' || c == L'\t' || c == L' ' || c == L'\r')
		{
			bGoodEndChar = false;
		}
	}

	return bGoodEndChar;
}

const TCHAR* GetConsoleVariableSetByName(EConsoleVariableFlags ConsoleVariableFlags)
{
	EConsoleVariableFlags SetBy = (EConsoleVariableFlags)((uint32)ConsoleVariableFlags & ECVF_SetByMask);

	#define CASE(x) case ECVF_SetBy##x: return TEXT(#x);
	switch(SetBy)
	{
		ENUMERATE_SET_BY(CASE)
	}
	#undef CASE
	
	return TEXT("<UNKNOWN>");
}

EConsoleVariableFlags GetConsoleVariableSetByValue(const TCHAR* SetByName)
{
	#define TEST(x) if (FCString::Stricmp(SetByName, TEXT(#x)) == 0) { return ECVF_SetBy##x; }
	ENUMERATE_SET_BY(TEST)
	#undef TEST
	
	return ECVF_SetByMask;
}


TArray<const FAutoConsoleObject*>& FAutoConsoleObject::AccessGeneralShaderChangeCvars()
{
	// This variable cannot be global because it is accessed from the constructor of other global variables.
	static TArray<const FAutoConsoleObject*> GeneralShaderChangeCvars;
	return GeneralShaderChangeCvars;
}

TArray<const FAutoConsoleObject*>& FAutoConsoleObject::AccessMobileShaderChangeCvars()
{
	// This variable cannot be global because it is accessed from the constructor of other global variables.
	static TArray<const FAutoConsoleObject*> MobileShaderChangeCvars;
	return MobileShaderChangeCvars;
}

TArray<const FAutoConsoleObject*>& FAutoConsoleObject::AccessDesktopShaderChangeCvars()
{
	// This variable cannot be global because it is accessed from the constructor of other global variables.
	static TArray<const FAutoConsoleObject*> DesktopShaderChangeCvars;
	return DesktopShaderChangeCvars;
}

class FConsoleVariableBase : public IConsoleVariable
{
public:
	/**
	 * Constructor
	 * @param InHelp must not be 0, must not be empty
	 */
	FConsoleVariableBase(const TCHAR* InHelp, EConsoleVariableFlags InFlags)
		:Flags(InFlags), bWarnedAboutThreadSafety(false)
	{
		SetHelp(InHelp);

		ApplyPreviewIfScalability();
	}

	void ApplyPreviewIfScalability()
	{
		if (((uint32)Flags & (uint32)ECVF_Scalability) != 0
			&& ((uint32)Flags & (uint32)ECVF_ExcludeFromPreview) == 0)
		{
			Flags = (EConsoleVariableFlags)((uint32)Flags | (uint32)ECVF_Preview);
		}
	}

	// interface IConsoleVariable -----------------------------------

#if UE_WITH_CVAR_HELP_TEXT
	virtual const TCHAR* GetHelp() const
	{
		return *Help;
	}
	
	virtual void SetHelp(const TCHAR* Value) override final
	{
		check(Value);

		Help = Value;

		// for now disabled as there is no good callstack when we crash early during engine init
//		ensure(IsGoodHelpString(Value));
	}

	virtual FText GetDetailedHelp() const override
	{
		// Append the current value of the variable to provide more details :
		const FString CurrentValueString = GetString();
		return FText::Format(LOCTEXT("CVarDetailedHelp", "{0}\n\nCurrent value : {1}"), FText::FromString(GetHelp()), CurrentValueString.IsEmpty() ? LOCTEXT("CVarEmptyValueString", "<empty>") : FText::FromString(CurrentValueString));
	}
#endif

	virtual EConsoleVariableFlags GetFlags() const
	{
		return Flags;
	}
	
	virtual void SetFlags(const EConsoleVariableFlags Value)
	{
		Flags = Value;
		ApplyPreviewIfScalability();
	}

	virtual class IConsoleVariable* AsVariable()
	{
		return this;
	}
	
	virtual void LogHistory(FOutputDevice& Ar)
	{
	
	}
	
	/** Legacy function to add old single delegates to the new multicast delegate. */
	virtual void SetOnChangedCallback(const FConsoleVariableDelegate& Callback) override
	{
		OnChangedCallback.Remove(LegacyDelegateHandle);
		LegacyDelegateHandle = OnChangedCallback.Add(Callback);
	}
	
	/** Remove a single delegate to the new multicast delegate. */
	virtual void ClearOnChangedCallback() override
	{
		OnChangedCallback.Remove(LegacyDelegateHandle);
	}

	/** Returns a multicast delegate with which to register. Called when this CVar changes. */
	virtual FConsoleVariableMulticastDelegate& OnChangedDelegate()
	{
		return OnChangedCallback;
	}

	// ------

	bool CanChange(EConsoleVariableFlags SetBy) const
	{
		uint32 OldPri =	(uint32)Flags & ECVF_SetByMask;
		uint32 NewPri =	(uint32)SetBy & ECVF_SetByMask;

		bool bRet = NewPri >= OldPri;

		if(!bRet)
		{
			FString CVarName = IConsoleManager::Get().FindConsoleObjectName(this);

			const FString Message = FString::Printf(TEXT("Setting the console variable '%s' with 'SetBy%s' was ignored as it is lower priority than the previous 'SetBy%s'. Value remains '%s'"),
				CVarName.IsEmpty() ? TEXT("unknown?") : *CVarName,
				GetConsoleVariableSetByName((EConsoleVariableFlags)NewPri),
				GetConsoleVariableSetByName((EConsoleVariableFlags)OldPri),
				*GetString()
				);
				
			if (OldPri == ECVF_SetByConsoleVariablesIni || OldPri == ECVF_SetByCommandline || OldPri == ECVF_SetBySystemSettingsIni || OldPri == ECVF_SetByHotfix)
			{
				// Set by an ini that has to be hand edited, a deliberate fail
				UE_LOGF(LogConsoleManager, Verbose, "%ls", *Message);
			}
			else if (NewPri == ECVF_SetByScalability && OldPri == ECVF_SetByDeviceProfile)
			{
				// Set by a device profile and attempted to update in scalability,
				// not a warning but very helpful to know when this happens
				UE_LOGF(LogConsoleManager, Log, "%ls", *Message);
			}
			else
			{
				// Treat as a warning unless we're intentionally ignoring this priority interaction		
				UE_LOGF(LogConsoleManager, Warning, "%ls", *Message);
			}
		}

		return bRet;
	}

	virtual void OnChanged(EConsoleVariableFlags SetBy, bool bForce)
	{	
		// SetBy can include set flags. Discard them here
		bool bIsSetOnly = SetBy & ECVF_Set_SetOnly_Unsafe;
		SetBy = EConsoleVariableFlags(SetBy & ~ECVF_SetFlagMask);

		// you have to specify a SetBy e.g. ECVF_SetByCommandline
		check(((uint32)SetBy & ECVF_SetByMask) || SetBy == ECVF_Default);

		// double check, if this fires we miss a if(CanChange(SetBy))
		check(bForce || CanChange(SetBy));

		// only change on main thread

		Flags = (EConsoleVariableFlags)(((uint32)Flags & ECVF_FlagMask) | SetBy);

		// we don't want to call the callback if SetOnly is used
		if (bIsSetOnly)
		{
			return;
		}

		OnChangedCallback.Broadcast(this);
	}

	
	// ------
	// Helper accessors to get to FConsoleVariableExtendedData, when we don't have a Type


#if ALLOW_OTHER_PLATFORM_CONFIG
	/**
	 * Internal function for caching data for other platforms - this is like Set(), but won't do any callbacks, or log any errors/warnings
	 */
	virtual void SetOtherPlatformValue(const TCHAR* InValue, EConsoleVariableFlags SetBy, FName Tag) = 0;
#endif

protected: // -----------------------------------------
#if UE_WITH_CVAR_HELP_TEXT
	// not using TCHAR* to allow chars support reloading of modules (otherwise we would keep a pointer into the module)
	FString Help;
#endif
	//
	EConsoleVariableFlags Flags;
	/** User function to call when the console variable is changed */
	FConsoleVariableMulticastDelegate OnChangedCallback;
	/** Store the handle to the delegate assigned via the legacy SetOnChangedCallback() so that the previous can be removed if called again. */
	FDelegateHandle LegacyDelegateHandle;

	/** True if this console variable has been used on the wrong thread and we have warned about it. */
	mutable bool bWarnedAboutThreadSafety;

	// @return 0:main thread, 1: render thread, later more
	uint32 GetShadowIndex() const
	{
		if((uint32)Flags & ECVF_RenderThreadSafe)
		{
			return IsInGameThread() ? 0 : 1;
		}
		else
		{
			FConsoleManager& ConsoleManager = GetManager();
			if(ConsoleManager.IsThreadPropagationThread() && FPlatformProcess::SupportsMultithreading())
			{
				if(!bWarnedAboutThreadSafety)
				{
					FString CVarName = ConsoleManager.FindConsoleObjectName(this);
					UE_LOGF(LogConsoleManager, Warning,
						"Console variable '%ls' used in the render thread. Rendering artifacts could happen. Use ECVF_RenderThreadSafe or don't use in render thread.",
						CVarName.IsEmpty() ? TEXT("unknown?") : *CVarName
						);
					bWarnedAboutThreadSafety = true;
				}
			}
			// other threads are not handled at the moment (e.g. sound)
		}
	
		return 0;
	}
};

class FConsoleCommandBase : public IConsoleCommand
{
public:
	/**
	 * Constructor
	 * @param InHelp must not be 0, must not be empty
	 */
	FConsoleCommandBase(const TCHAR* InHelp, EConsoleVariableFlags InFlags) :
#if UE_WITH_CVAR_HELP_TEXT
		Help(InHelp),
#endif
		Flags(InFlags)
	{
#if UE_WITH_CVAR_HELP_TEXT
		check(InHelp);
		//check(*Help != 0); for now disabled as there callstack when we crash early during engine init
#endif
		ApplyPreviewIfScalability();
	}

	void ApplyPreviewIfScalability()
	{
		if (((uint32)Flags & (uint32)ECVF_Scalability) != 0
			&& ((uint32)Flags & (uint32)ECVF_ExcludeFromPreview) == 0)
		{
			Flags = (EConsoleVariableFlags)((uint32)Flags | (uint32)ECVF_Preview);
		}
	}

	// interface IConsoleVariable -----------------------------------

#if UE_WITH_CVAR_HELP_TEXT
	virtual const TCHAR* GetHelp() const
	{
		return *Help;
	}

	virtual void SetHelp(const TCHAR* InValue)
	{
		check(InValue);
		check(*InValue != 0);

		Help = InValue;
	}
#endif

	virtual EConsoleVariableFlags GetFlags() const
	{
		return Flags;
	}

	virtual void SetFlags(const EConsoleVariableFlags Value)
	{
		Flags = Value;
		ApplyPreviewIfScalability();
	}

	virtual struct IConsoleCommand* AsCommand()
	{
		return this;
	}

private: // -----------------------------------------
#if UE_WITH_CVAR_HELP_TEXT
	// not using TCHAR* to allow chars support reloading of modules (otherwise we would keep a pointer into the module)
	FString Help;
#endif
	EConsoleVariableFlags Flags;
};

template <class T>
void OnCVarChange(T& Dst, const T& Src, EConsoleVariableFlags Flags, EConsoleVariableFlags SetBy)
{
	// for the SetOnly case, just copy over the source to the dest
	if (SetBy & ECVF_Set_SetOnly_Unsafe)
	{
		Dst = Src;
		return;
	}
	
	FConsoleManager& ConsoleManager = GetManager();

#if WITH_RELOAD
	// Unlike HotReload, Live Coding does global initialization outside of the main thread.  During global initialization,
	// Live Coding does have the main thread stalled so there is a "reduced" chance of threading issues.  During global
	// initialization with live coding, this code only gets called when a new instance of an existing CVar is created.
	// Normally this shouldn't happen unless a source file shuffles between two different unity files.
	if (IsInGameThread() || IsReloadActive())
#else
	if(IsInGameThread())
#endif
	{
		if((Flags & ECVF_RenderThreadSafe) && ConsoleManager.GetThreadPropagationCallback())
		{
			// defer the change to be in order with other rendering commands
			ConsoleManager.GetThreadPropagationCallback()->OnCVarChange(Dst, Src);
		}
		else
		{
			// propagate the change right away
			Dst = Src;
		}
	}
	else
	{
		// CVar Changes can only be initiated from the main thread
		check(0);
	}

	if ((SetBy & ECVF_Set_NoSinkCall_Unsafe) == 0)
	{
		ConsoleManager.OnCVarChanged();
	}
}

#if ALLOW_OTHER_PLATFORM_CONFIG

static void ExpandScalabilityCVar(FConfigCacheIni* ConfigSystem, const FString& CVarKey, const FString& CVarValue, TMap<FString, FString>& ExpandedCVars, bool bOverwriteExistingValue)
{
	// load scalability settings directly from ini instead of using scalability system, so as not to inadvertantly mess anything up
	// if the DP had sg.ResolutionQuality=3, we would read [ResolutionQuality@3]
	FString SectionName = FString::Printf(TEXT("%s@%s"), *CVarKey.Mid(3), *CVarValue);
	// walk over the scalability section and add them in, unless already done
	const FConfigSection* ScalabilitySection = ConfigSystem->GetSection(*SectionName, false, GScalabilityIni);
	if (ScalabilitySection != nullptr)
	{
		for (const auto& Pair : *ScalabilitySection)
		{
			FString ScalabilityKey = Pair.Key.ToString();
			if (bOverwriteExistingValue || !ExpandedCVars.Contains(ScalabilityKey))
			{
				ExpandedCVars.Add(ScalabilityKey, Pair.Value.GetValue());
			}
		}
	}
}

EConsoleVariableFlags GetPreviewFlagsOfCvar(const TCHAR* Name)
{
	// now look up the cvar, if it exists (it's okay if it doesn't, it may not exist on host platform, but then it's not previewable!)
	IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(Name, false /* bTrackFrequentCalls */);
	EConsoleVariableFlags PreviewFlag = (CVar != nullptr) ? (EConsoleVariableFlags)(CVar->GetFlags() & ECVF_Preview) : ECVF_Default;
	return PreviewFlag;
}

bool IConsoleManager::VisitPlatformCVarsForEmulation(FName PlatformName, const FString& DeviceProfileName, TFunctionRef<void(const FString& CVarName, const FString& CVarValue, EConsoleVariableFlags SetBy)> Visit)
{
	// we can't get to Scalability code in here (it's in Engine) but we still want to apply the default level... 
	// there is a static_assert in UDeviceProfile::ExpandDeviceProfileCVars to make sure the default doesn't change from this value
	const int DefaultScalabilityLevel = 3;

	// get the config system for the platform the DP uses
	FConfigCacheIni* ConfigSystem = FConfigCacheIni::ForPlatform(*PlatformName.ToString());
	if (ConfigSystem == nullptr)
	{
		return false;
	}

	// ECVF_SetByConstructor:
	//   doesn't come from ini

	// ECVF_SetByScalability:
	//	 initializes sg. cvars to the Default level, likely DPs etc will replace most/all of these settings

	// ECVF_SetByProjectSetting:
	// ECVF_SetBySystemSettingsIni:
	//   read from ini files
	
	// ECVF_SetByDeviceProfile: 
	//   read from the DP parenting chain

	// ECVF_SetByGameSetting:
	//   skipped, since we don't have a user

	// ECVF_SetByConsoleVariablesIni:
	//   maybe skip this? it's a weird one, but maybe?

	// ECVF_SetByCommandline:
	//   skip as this would not be expected to apply to emulation

	// ECVF_SetByCode:
	//   skip because it cannot be set by code

	// ECVF_SetByConsole
	//   we could have this if we made a per-platform CVar, not just the shared default value

	const TCHAR* DeviceProfileTag = TEXT("_NamedDeviceProfile");
	const TCHAR* ScalabilityTag = TEXT("_Scalability");
	struct FSectionPair
	{
		const TCHAR* Name; EConsoleVariableFlags SetBy;
	} Sections[] =
	{
		// this order is not based on priority order, but by the order as seen in LaunchEngineLoop:
		//		GSystemSettings.Initialize()
		{ TEXT("SystemSettings"), ECVF_SetBySystemSettingsIni },

		//		ApplyCVarSettingsFromIni(...)
		{ TEXT("/Script/Engine.RendererSettings"), ECVF_SetByProjectSetting },
		{ TEXT("/Script/Engine.RendererOverrideSettings"), ECVF_SetByProjectSetting },
		{ TEXT("/Script/Engine.StreamingSettings"), ECVF_SetByProjectSetting },
		{ TEXT("/Script/Engine.GarbageCollectionSettings"), ECVF_SetByProjectSetting },
		{ TEXT("/Script/Engine.NetworkSettings"), ECVF_SetByProjectSetting },

		//		Scalability::InitScalabilitySystem()
		{ ScalabilityTag, ECVF_SetByDeviceProfile },

		//		UDeviceProfileManager::InitializeCVarsForActiveDeviceProfile()
		{ DeviceProfileTag, ECVF_SetByDeviceProfile },

		//		FConfigCacheIni::LoadConsoleVariablesFromINI()
		{ TEXT("Startup"), ECVF_SetByConsoleVariablesIni },
		{ TEXT("ConsoleVariables"), ECVF_SetBySystemSettingsIni },

		// Preview active GameFeaturePlugins - 
		// NOTE: THIS ONLY WORKS FOR ENGINE.INI GFP CONFIG FILES!
		//       because the code below only looks in the Engine branch, the GFPs don't run their Dynamic Modification code on other platforms configs
		{ TEXT("PluginCVars_LowPriority"), ECVF_SetByPluginLowPriority },
		{ TEXT("PluginCVars_HighPriority"), ECVF_SetByPluginHighPriority },

	};

	TMap<FString, int32> CVarSetByMap;
	auto VisitIfAllowed = [&CVarSetByMap, &Visit](const FString& InName, const FString& Value, EConsoleVariableFlags SetBy)
	{
		FString Name = InName;
		// get the name of the real variable for any shadows
		IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(*Name, false /* bTrackFrequentCalls */);
		if (CVar && CVar->IsShadowObject())
		{
			Name = IConsoleManager::Get().FindConsoleObjectName(CVar->GetShadowedObject());
		}

		int32 SetByInt = (int32)(SetBy & ECVF_SetByMask);
		int32 WasSetBy = CVarSetByMap.FindOrAdd(Name);
		if (SetByInt < WasSetBy)
		{
			UE_LOGF(LogConsoleManager, Log, "Skipping CVar %ls=%ls while visiting another platform, because it was already visited with a higher priority", *Name, *Value);
			return;
		}

		CVarSetByMap[Name] = SetByInt;

		Visit(Name, Value, (EConsoleVariableFlags)(SetBy | GetPreviewFlagsOfCvar(*Name)));
	};

	// now walk up the stack getting current values
	for (const FSectionPair& SectionPair : Sections)
	{
		bool bDeleteSection = false;
		const FConfigSection* Section;
		bool bIsDeviceProfile = FCString::Strcmp(SectionPair.Name, DeviceProfileTag) == 0;
		bool bIsScalabilityLevel = FCString::Strcmp(SectionPair.Name, ScalabilityTag) == 0;

		if (bIsDeviceProfile)
		{
			// skip this if we didn't specify one
			if (DeviceProfileName.Len() == 0)
			{
				continue;
			}
			if (!FCoreDelegates::GatherDeviceProfileCVars.IsBound())
			{
				UE_LOGF(LogConsoleManager, Warning, "Attempted to get CVars for another platform before FCoreDelegates::GatherDeviceProfileCVars was bound to a callback. CVar values are likely incorrect.");
				continue;
			}

			// make a fake section of the dp cvars - this will let the expansion happen as normal below
			FConfigSection* NewSection = new FConfigSection();
			bDeleteSection = true;

			UE_LOGF(LogConsoleManager, Verbose, "Gathering device profile cvars for %ls, platform config %ls", *DeviceProfileName, *PlatformName.ToString());
			// run the delegate (this code can't get into DP code directly, so we use a delegate), and walk over the results
			for (TPair<FName, FString>& Pair : FCoreDelegates::GatherDeviceProfileCVars.Execute(DeviceProfileName))
			{
				NewSection->Add(Pair);
				UE_LOGF(LogConsoleManager, Verbose, "   %ls = %ls", *Pair.Key.ToString(), *Pair.Value);
			}
            
            Section = NewSection;
		}
		else if (bIsScalabilityLevel)
		{
			// walk over all of the scalability groups, and assign them to the default level
			FString DefaultLevel = FString::Printf(TEXT("%d"), DefaultScalabilityLevel);

			// make a fake section of the sg. vars - this will let the expansion happen as normal below
            FConfigSection* NewSection = new FConfigSection();
			bDeleteSection = true;
			IConsoleManager::Get().ForEachConsoleObjectThatStartsWith(FConsoleObjectVisitor::CreateLambda([&DefaultLevel, NewSection](const TCHAR* Name, IConsoleObject* Obj)
				{
					if (Obj->TestFlags(ECVF_ScalabilityGroup))
					{
						NewSection->Add(Name, FConfigValue(DefaultLevel));
					}
				}
			));
            
            Section = NewSection;
		}
		else
		{
			static const FString ConsoleVariablesIni = FPaths::EngineDir() + TEXT("Config/ConsoleVariables.ini");
			const FString& IniFile = (SectionPair.SetBy == ECVF_SetByConsoleVariablesIni) ? ConsoleVariablesIni : GEngineIni;
			Section = ConfigSystem->GetSection(SectionPair.Name, false, IniFile);
		}


		if (Section != nullptr)
		{
			// add the cvars from the section
			for (const auto& Pair : *Section)
			{
				FString Key = Pair.Key.ToString();
				FString Value = Pair.Value.GetValue();

				// don't bother tracking when looking up other platform cvars
				EConsoleVariableFlags PreviewFlag = GetPreviewFlagsOfCvar(*Key);

				if (Key.StartsWith(TEXT("sg.")))
				{
					// @todo ini: If anything in here was already set, overwrite it or skip it?
					// the priorities may cause runtime to fail to set a cvar that this will set blindly, since we are ignoring
					// priority by doing them "in order". Scalablity is one of the lowest priorities, so should almost never be allowed?

					TMap<FString, FString> ScalabilityCVars;
					ExpandScalabilityCVar(ConfigSystem, Key, Value, ScalabilityCVars, true);

					for (const auto& ScalabilityPair : ScalabilityCVars)
					{
						// See if the expanded scalability cvar is not allowed to preview and has ECVF_ExcludeFromPreview set
						IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(*ScalabilityPair.Key, false /* bTrackFrequentCalls */);
						EConsoleVariableFlags ScalabilityCvarFlags = (CVar != nullptr) ? CVar->GetFlags() : ECVF_Default;
						if (ScalabilityCvarFlags & ECVF_ExcludeFromPreview)
						{
							VisitIfAllowed(ScalabilityPair.Key, ScalabilityPair.Value, (EConsoleVariableFlags)(ECVF_SetByScalability));
						}
						else
						{
							VisitIfAllowed(ScalabilityPair.Key, ScalabilityPair.Value, (EConsoleVariableFlags)(ECVF_SetByScalability | PreviewFlag));
						}
					}
				}

				// run the callback with all cvars, even scalbility groups
				VisitIfAllowed(Key, Value, (EConsoleVariableFlags)(SectionPair.SetBy | PreviewFlag));
			}

			// clean up the temp section we made
			if (bDeleteSection)
			{
				delete Section;
			}

		}
	}


	return true;
}


#endif


constexpr bool IsArrayPriority(EConsoleVariableFlags Priority)
{
	Priority = (EConsoleVariableFlags)(Priority & ECVF_SetByMask);
	return
		Priority == ECVF_SetByPluginLowPriority ||
		Priority == ECVF_SetByPluginHighPriority ||
		Priority == ECVF_SetByHotfix ||
		Priority == ECVF_SetByDeviceProfile ||
		Priority == ECVF_SetByScalability ||
		Priority == ECVF_SetByPreview ||
		Priority == ECVF_SetByTemp;
}

template <class T>
class FConsoleVariableHistory
{
public:
	using FHistoryData = TConsoleVariableData<T>;
	using FTaggedHistoryData = TPair<FName, FHistoryData>;
	TSortedMap<int, TArray<FTaggedHistoryData>> History;
	
	bool bHasTaggedArrayData = false;

	void FindHighestPriorityAndTag(
		EConsoleVariableFlags MinPriority,
		EConsoleVariableFlags MaxPriority,
		EConsoleVariableFlags& OutFlags, 
		FName& OutTag)
	{
		OutTag = NAME_None;
		OutFlags = MinPriority;

		for (typename TSortedMap<int, TArray<FTaggedHistoryData>>::TReverseIterator It(History); It; ++It)
		{
			if (It.Value().Num() > 0)
			{
				if (((EConsoleVariableFlags)It.Key() <= MaxPriority) &&
					((EConsoleVariableFlags)It.Key() >= MinPriority))
				{
					OutTag = It.Value().Last().Key;
					OutFlags = (EConsoleVariableFlags)(It.Key());
					// Replacing a value that is ECVF_SetByConstructor is not allowed.
					if (OutFlags == ECVF_SetByConstructor)
					{
						OutFlags = SETBY_ERROR;
					}
					return;
				}
			}
		}
	}

	/**
	 * When we change a vlaue, we track the change at each priority level, so we can unset them later. 
	 * @return false if this changed a value in place and it is not the final level (in array type priorities). 
	 * This is used to not fire unnecessary callbacks in this case:
	 *   PluginHighPriority -> Tag1=Foo, Tag2=Bar, and we are setting Tag1 to Baz
	 * the value of the cvar cannot change, and when we update in place we are trying to minimize callbacks
	 */
	bool Track(const TCHAR* InValue, EConsoleVariableFlags Flags, FName Tag)
	{
		T LocalCopy;
		UE::ConsoleManager::Private::GetValueFromString<T>(LocalCopy, UE::ConfigUtilities::ConvertValueFromHumanFriendlyValue(InValue));
		
		EConsoleVariableFlags SetBy = (EConsoleVariableFlags)(Flags & ECVF_SetByMask);
		TArray<FTaggedHistoryData>& ValueArray = History.FindOrAdd((int)SetBy);
		
		// if desired, set in place, looking for existing tag, and replacing it if it's there
		// otherwise always add it to the end further down
		if (Flags & ECVF_Set_ReplaceExistingTag)
		{
			for (int32 Index = 0; Index < ValueArray.Num(); Index++)
			{
				FTaggedHistoryData& HistoryItem = ValueArray[Index];
				if (HistoryItem.Key == Tag)
				{
					HistoryItem.Value = LocalCopy;
					// if we aren't the last entry, then we don't want to allow any callbacks to fire
					// (the CanChange check later on just checks against SetBy level, not by array index)
					return Index == ValueArray.Num() - 1;
				}
			}
		}

		if (!IsArrayPriority(SetBy) && ValueArray.Num() > 0)
		{
			// replace the value if it already exists here
			ValueArray[0].Value = LocalCopy;
		}
		else
		{
			// Tag's in array types must be distinct.
			// If the Tag already exists, we remove it and add it at the end.

			for (int32 Index = 0; Index < ValueArray.Num(); Index++)
			{
				if (ValueArray[Index].Key == Tag)
				{
					ValueArray.RemoveAt(Index);
					break;
				}
			}

			// add the value to the array
			ValueArray.Emplace(Tag, LocalCopy);
		}

		return true;
	}
	
	/**
	 * Unset the value at the SetBy, and optionally Tag.
	 * Return true if it had been set, or false if nothing happened
	 */
	bool Unset(EConsoleVariableFlags Flags, FName Tag)
	{
		EConsoleVariableFlags SetBy = (EConsoleVariableFlags)(Flags & ECVF_SetByMask);

		if (!History.Contains(SetBy))
		{
			return false;
		}
		
		TArray<FTaggedHistoryData>& ValueArray = History[SetBy];

		bool bUnsetSomething = false;

		// first remove the value (if no tag, remove all of them)
		if (Tag != NAME_None)
		{
			if (IsArrayPriority(SetBy))
			{
				// look for the tag in the list
				for (auto It = ValueArray.CreateIterator(); It; ++It)
				{
					if (It->Key == Tag)
					{
						bUnsetSomething = true;
						It.RemoveCurrent();
					}
				}

				// toss it if it's now empty
				if (ValueArray.IsEmpty())
				{
					History.Remove(SetBy);
				}
			}
			else if (ValueArray.Num() > 0 && ValueArray[0].Key == Tag)
			{
				bUnsetSomething = true;
				History.Remove(SetBy);
			}
		}
		else if (ValueArray.Num() > 0)
		{
			bUnsetSomething = true;
			History.Remove(SetBy);
		}

		return bUnsetSomething;
	}
	
	const FHistoryData& GetMaxValue(EConsoleVariableFlags& MaxSetBy)
	{
		for (typename TSortedMap<int, TArray<FTaggedHistoryData>>::TReverseIterator It(History); It; ++It)
		{
			if (It.Value().Num() > 0)
			{
				MaxSetBy = (EConsoleVariableFlags)It.Key();
				return It.Value().Last().Value;
			}
		}
		
		// indicate failed to find one
		MaxSetBy = ECVF_SetByMask;
		static FHistoryData Default(T{});
		return Default;
	}
	
	void Log(FOutputDevice& Ar)
	{
		Ar.Logf(TEXT("\nHISTORY"));
		int Prio = 0;
		for (auto& Pri : History)
		{
			for (FTaggedHistoryData& Pair : Pri.Value)
			{
				FString KeyStr = FString();
				
				Ar.Logf(TEXT("%s: %s%s"),
					GetConsoleVariableSetByName((EConsoleVariableFlags)(Pri.Key)),
					*TTypeToString<T>::ToString(Pair.Value.GetValueOnGameThread()),
					(Pair.Key != NAME_None) ? *FString::Printf(TEXT(" [%s]"), *Pair.Key.ToString()) : TEXT(""));
			}
			Prio++;
		}
	}
};


// an intermediate class between specific typed CVars and FConsoleVariableBase to handle history and (in some configurations)
// cached values of the CVar on other platforms/device profiles. It is expected that all CVar classes extend from this (if not
// you will get abtract class compilation errors). We need the store type T, so we cannot put this up into FConsoleVariableBase
template <class T>
class FConsoleVariableExtendedData : public FConsoleVariableBase
{
public:

	FConsoleVariableExtendedData(const T& DefaultValue, const TCHAR* Help, EConsoleVariableFlags Flags);

	virtual ~FConsoleVariableExtendedData()
	{
		// if we had been put into any tagged
		if (PriorityHistory != nullptr && PriorityHistory->bHasTaggedArrayData)
		{
			// make sure we are pulled out of the global list of tagged cvars
			for (auto Pair : UE::ConsoleManager::Private::TaggedCVars)
			{
				Pair.Value->Remove(this);
			}
			
			delete PriorityHistory;
		}
	}
	
	virtual FResolvedContext ResolveContext(const FSetContext& Context) override
	{
		FResolvedContext Resolved;
		Resolved.Flags = Context.Flags;
		Resolved.Tag = Context.Tag;

		EConsoleVariableFlags HighestSetBy = ECVF_SetByConstructor;
		FName HighestTag = NAME_None;

		if ((Context.PriorityMode == FSetContext::EPriorityMode::ReplaceCurrent) ||
			(Context.TagMode == FSetContext::ETagMode::ReplaceCurrent))
		{
			if (PriorityHistory == nullptr)
			{
				PriorityHistory = new FConsoleVariableHistory<T>();
				PriorityHistory->Track(*GetString(), ECVF_SetByConstructor, NAME_None);
			}

			PriorityHistory->FindHighestPriorityAndTag(
				Context.MinPriority,
				Context.MaxPriority,
				HighestSetBy,
				HighestTag);
		}
		
		if (Context.PriorityMode == FSetContext::EPriorityMode::ReplaceCurrent)
		{
			EConsoleVariableFlags ExtraFlags = (EConsoleVariableFlags)(Context.Flags & ~ECVF_SetByMask);
			Resolved.Flags = (EConsoleVariableFlags)(HighestSetBy | ExtraFlags);
			if (HighestSetBy == SETBY_ERROR)
			{
				UE_LOGF(LogConsoleManager, Display, 
					"Trying to Replace Cvar=%ls Set By Constructor Value implicitly by calling SetWithCurrentPriority or ReplaceCurrentPriorityAndTag.\n" "This is not allowed, possible solutions:\n" "1. Use SetWithCurrentPriority or ReplaceCurrentPriorityAndTag with explicit MinPriority and MaxPriority values, with MinPriority >= ECVF_SetByScalability.\n" "2. Use explicit Set, Cvar->Set(<Value>, <SetByPriority>)",
					*IConsoleManager::Get().FindConsoleObjectName(this));
			}
		}

		if (Context.TagMode == FSetContext::ETagMode::ReplaceCurrent)
		{
			Resolved.Tag = HighestTag;
		}
		
		if ((Context.TagMode == FSetContext::ETagMode::Replace))
		{
			Resolved.Flags = (EConsoleVariableFlags)(Resolved.Flags | ECVF_Set_ReplaceExistingTag);
		}

		return Resolved;
	}

	/**
	 * Subclasses implement this to simply set TypedValue as the current value for the CVar. It must not call any callbacks
	 */
	virtual void SetInternal(const T& TypedValue, EConsoleVariableFlags SetBy) = 0;

	/**
	 * Similar to the Set function, except that it must always work, independent of priority. This is used by the History system
	 * to set the value wth a lower priority, and then update the priority.
	 */
	void SetInternalAndUpdateState(const T& TypedValue, EConsoleVariableFlags SetBy)
	{
		SetInternal(TypedValue, SetBy);
		OnChanged(SetBy, true);
	}

#if ALLOW_OTHER_PLATFORM_CONFIG

	virtual TSharedPtr<IConsoleVariable> GetPlatformValueVariable(FName PlatformName, const FString& DeviceProfileName) override;
	virtual bool HasPlatformValueVariable(FName PlatformName, const FString& DeviceProfileName) override;
	virtual void ClearPlatformVariables(FName PlatformName) override;

	// cache of the values of this cvar on other platforms
	TMap<FName, TSharedPtr<IConsoleVariable> > PlatformValues;
	FRWLock PlatformValuesLock;
	friend FConsoleManager;

#endif

	virtual FString GetDefaultValue() override
	{
		// simply convert the default typed value to a string
		return TTypeToString<T>::ToString(GetDefaultTypedValue());
	}

	T GetDefaultTypedValue()
	{
		// pull our constructed value out of the history if it exists, (if it doesn't, than our current value is the constructed value!)
		if (PriorityHistory != nullptr)
		{
			// constructor will never have more than one value
			return PriorityHistory->History[ECVF_SetByConstructor][0].Value.GetValueOnAnyThread();
		}
		
		// if we have no history at all, that means we never called Set, so the current value must be the value we were constructed with
		T ConstructorValue;
		GetValue(ConstructorValue);
		return ConstructorValue;
	}

protected:
	// A history object will be created and set here the first time Set() is called to change the value
	FConsoleVariableHistory<T>* PriorityHistory = nullptr;
	friend class FConsoleManager;

#if ALLOW_OTHER_PLATFORM_CONFIG
	virtual void SetOtherPlatformValue(const TCHAR* InValue, EConsoleVariableFlags SetBy, FName Tag) override
	{
		// always track it
		if (TrackHistory(InValue, SetBy, Tag) == false)
		{
			return;
		}

		// set it if we are equal to or higher than before
		uint32 CurrentSetBy = GetFlags() & ECVF_SetByMask;
		uint32 NewSetBy = SetBy & ECVF_SetByMask;

		if (NewSetBy >= CurrentSetBy)
		{
			// update value
			T ConvertedValue;
			UE::ConsoleManager::Private::GetValueFromString<T>(ConvertedValue, UE::ConfigUtilities::ConvertValueFromHumanFriendlyValue(InValue));
			// set the value, and push to render thread value as well, but don't trigger callbacks and don't check priorties
			SetInternalAndUpdateState(ConvertedValue, (EConsoleVariableFlags)(SetBy | ECVF_Set_SetOnly_Unsafe));

			// update the setby
			SetFlags((EConsoleVariableFlags)((GetFlags() & ~ECVF_SetByMask) | NewSetBy));
		}
	}
#endif

	/**
	 * This is the key function that subclasses need to call in their Set() implementation. This will track the values of cvars at
	 * different priorities, so priorities/plugins/etc can be unset later, and the CVar will updatae state correctly (in Unset())
	 */
	bool TrackHistory(const TCHAR* InValue, EConsoleVariableFlags SetBy, FName Tag)
	{
		// make a history if we want to
		if (PriorityHistory == nullptr)
		{
			PriorityHistory = new FConsoleVariableHistory<T>();
			PriorityHistory->Track(*GetString(), ECVF_SetByConstructor, NAME_None);
		}

		if (PriorityHistory->Track(InValue, SetBy, Tag) == false)
		{
			return false;
		}

		// this cvar so we can remove ourself later
		if (Tag != NAME_None)
		{
			TSet<IConsoleVariable*>* TaggedSet = UE::ConsoleManager::Private::TaggedCVars.FindRef(Tag);
			if (TaggedSet == nullptr)
			{
				TaggedSet = UE::ConsoleManager::Private::TaggedCVars.Add(Tag, new TSet<IConsoleVariable*>());
			}
			TaggedSet->Add(this);
			
			// set a flag to remember we need to remove ourself from TaggedCVars in destructor
			PriorityHistory->bHasTaggedArrayData = true;
		}

		return true;
	}
	
	virtual SIZE_T GetHistorySize() override
	{
		if (PriorityHistory != nullptr)
		{
			return PriorityHistory->History.GetAllocatedSize();
		}
		return 0;
	}

	void LogHistory(FOutputDevice& Ar)
	{
		if (PriorityHistory != nullptr)
		{
			PriorityHistory->Log(Ar);
		}
	}

	bool PreprocessSet(const TCHAR* InValue, EConsoleVariableFlags SetBy, FName Tag)
	{
		// FILL IN THE NEW RENDERTHREAD STUFF HERE

		if (TrackHistory(InValue, SetBy, Tag) == false)
		{
			return false;
		}

		if (CanChange(SetBy) == false)
		{
			return false;
		}

		return true;
	}

	
	/**
	 * Removes the value at the given SetBy (and potentially Tag for the Array type SetBy priorities). This will update the
	 * current value of the CVar as needed (if the Unset is at the current prio)
	 */
	virtual void Unset(EConsoleVariableFlags SetBy, FName Tag) override
	{
		if (PriorityHistory == nullptr)
		{
			return;
		}
		
		uint32 UnsetFlags = (uint32)SetBy & ECVF_SetFlagMask;

		// if we don't know the SetBy, unset from all
		// this isn't ideal because it could call SetInternal multiple times
		if ((SetBy & ECVF_SetByMask) == ECVF_SetByMask)
		{
			#define RECURSE(x) Unset(EConsoleVariableFlags(ECVF_SetBy##x | UnsetFlags), Tag);
			ENUMERATE_SET_BY(RECURSE)
			#undef RECURSE
			
			return;
		}
		
		// if nothing was unset, no need to perform any more actions
		if (PriorityHistory->Unset(SetBy, Tag) == false)
		{
			return;
		}
		
		uint32 CurrentPri =	(uint32)this->Flags & ECVF_SetByMask;
		uint32 UnsetPri =	(uint32)SetBy & ECVF_SetByMask;

		// if we are unsetting at the current setby (or maybe in some weird cases, greater than setby) then we need to reset the SetBy and current value
		if (UnsetPri >= CurrentPri)
		{
			// now figure out the new value
			EConsoleVariableFlags NewSetBy;
			auto MaxValue = PriorityHistory->GetMaxValue(NewSetBy);

			// when we preview SGs, we set their value, but dont run the callbacks, so here we are doing the same operation
			if (GetFlags() & ECVF_ScalabilityGroup)
			{
				UnsetFlags |= ECVF_Set_SetOnly_Unsafe;
			}

			NewSetBy = (EConsoleVariableFlags)(NewSetBy | UnsetFlags);

			UE_LOGF(LogConsoleManager, Verbose, " |-> Unsetting %ls, now %ls", *IConsoleManager::Get().FindConsoleObjectName(this),
				*TTypeToString<T>::ToString(MaxValue.GetValueOnGameThread()));

			// and force it to the new value and call any set callbacks
			SetInternalAndUpdateState(MaxValue.GetValueOnGameThread(), NewSetBy);
		}
	}

};

template<class T>
class FConsoleVariableConversionHelper
{
public:
	static bool GetBool(T Value);
	static int32 GetInt(T Value);
	static float GetFloat(T Value);
	static FString GetString(T Value);
};

// specialization for bool
template<> bool FConsoleVariableConversionHelper<bool>::GetBool(bool Value)
{
	return Value;
}
template<> int32 FConsoleVariableConversionHelper<bool>::GetInt(bool Value)
{
	return Value ? 1 : 0;
}
template<> float FConsoleVariableConversionHelper<bool>::GetFloat(bool Value)
{
	return Value ? 1.0f : 0.0f;
}
template<> FString FConsoleVariableConversionHelper<bool>::GetString(bool Value)
{
	return Value ? TEXT("true") : TEXT("false");
}

// specialization for int32
template<> bool FConsoleVariableConversionHelper<int32>::GetBool(int32 Value)
{
	return Value != 0;
}
template<> int32 FConsoleVariableConversionHelper<int32>::GetInt(int32 Value)
{
	return Value;
}
template<> float FConsoleVariableConversionHelper<int32>::GetFloat(int32 Value)
{
	return (float)Value;
}
template<> FString FConsoleVariableConversionHelper<int32>::GetString(int32 Value)
{
	return FString::Printf(TEXT("%d"), Value);
}

// specialization for float
template<> bool FConsoleVariableConversionHelper<float>::GetBool(float Value)
{
	return Value != 0.0f;
}
template<> int32 FConsoleVariableConversionHelper<float>::GetInt(float Value)
{
	return (int32)Value;
}
template<> float FConsoleVariableConversionHelper<float>::GetFloat(float Value)
{
	return Value;
}
template<> FString FConsoleVariableConversionHelper<float>::GetString(float Value)
{
	UE::ConsoleManager::Private::FConsoleManagerLocaleScope LocaleScope;
	return FString::Printf(TEXT("%g"), Value);
}

// specialization for FString
template<> bool FConsoleVariableConversionHelper<FString>::GetBool(FString Value)
{
	bool OutValue = false;
	TTypeFromString<bool>::FromString(OutValue, *Value);
	return OutValue;
}
template<> int32 FConsoleVariableConversionHelper<FString>::GetInt(FString Value)
{
	int32 OutValue = 0;
	TTypeFromString<int32>::FromString(OutValue, *Value);
	return OutValue;
}
template<> float FConsoleVariableConversionHelper<FString>::GetFloat(FString Value)
{
	float OutValue = 0.0f;
	UE::ConsoleManager::Private::GetValueFromString<float>(OutValue, *Value);
	return OutValue;
}
template<> FString FConsoleVariableConversionHelper<FString>::GetString(FString Value)
{
	return Value;
}


// T: bool, int32, float, FString
template <class T>
class FConsoleVariable : public FConsoleVariableExtendedData<T>
{
// help find functions without needing this-> prefixes
	using FConsoleVariableBase::GetShadowIndex;
	using FConsoleVariableBase::CanChange;
	using FConsoleVariableBase::Flags;

public:
	FConsoleVariable(T DefaultValue, const TCHAR* Help, EConsoleVariableFlags Flags, IConsoleVariable* Parent=nullptr)
		: FConsoleVariableExtendedData<T>(DefaultValue, Help, Flags)
		, Data(DefaultValue)
#if ALLOW_OTHER_PLATFORM_CONFIG
		, ParentVariable(Parent)
#endif
	{
	}

	// interface IConsoleVariable ----------------------------------- 

	virtual void Release()
	{
		delete this; 
	}
	
	virtual void Set(const TCHAR* Value, const FConsoleVariableBase::FResolvedContext& Context) override
	{
		// if it was already set in the history (this returns false if nothing needs to be done), then skip the Set
		if (this->TrackHistory(Value, Context.Flags, Context.Tag) == false)
		{
			return;
		}

		if (CanChange(Context.Flags))
		{
			UE::ConsoleManager::Private::GetValueFromString<T>(Data.ShadowedValue[0], Value);
			OnChanged(Context.Flags, false);
		}
	}

	virtual bool GetBool() const override { return FConsoleVariableConversionHelper<T>::GetBool(Value()); }
	virtual int32 GetInt() const override { return FConsoleVariableConversionHelper<T>::GetInt(Value()); }
	virtual float GetFloat() const override { return FConsoleVariableConversionHelper<T>::GetFloat(Value()); }
	virtual FString GetString() const override { return FConsoleVariableConversionHelper<T>::GetString(Value()); }

	virtual bool IsVariableBool() const override { return false; }
	virtual bool IsVariableInt() const override { return false; }
	virtual bool IsVariableFloat() const override { return false; }
	virtual bool IsVariableString() const override { return false; }

	virtual class TConsoleVariableData<bool>* AsVariableBool() override { return nullptr; }
	virtual class TConsoleVariableData<int32>* AsVariableInt() override { return nullptr; }
	virtual class TConsoleVariableData<float>* AsVariableFloat() override { return nullptr; }
	virtual class TConsoleVariableData<FString>* AsVariableString() override { return nullptr; }

#if ALLOW_OTHER_PLATFORM_CONFIG
	virtual IConsoleObject* GetParentObject() const
	{
		return ParentVariable;
	}
#endif

private: // ----------------------------------------------------

	TConsoleVariableData<T> Data;
#if ALLOW_OTHER_PLATFORM_CONFIG
	IConsoleVariable* ParentVariable;
#endif

	const T &Value() const
	{
		// remove const
		FConsoleVariable<T>* This = (FConsoleVariable<T>*)this;
		return This->Data.GetReferenceOnAnyThread();
	}
	
	virtual void SetInternal(const T& TypedValue, EConsoleVariableFlags SetBy)
	{
		Data.ShadowedValue[0] = TypedValue;
	}

	virtual void OnChanged(EConsoleVariableFlags SetBy, bool bForce=false)
	{
		// propagate from main thread to render thread
		OnCVarChange(Data.ShadowedValue[1], Data.ShadowedValue[0], Flags, SetBy);
		FConsoleVariableBase::OnChanged(SetBy, bForce);
	}
};

// specialization for all

template<> bool FConsoleVariable<bool>::IsVariableBool() const
{
	return true;
}
template<> bool FConsoleVariable<int32>::IsVariableInt() const
{
	return true;
}
template<> bool FConsoleVariable<float>::IsVariableFloat() const
{
	return true;
}
template<> bool FConsoleVariable<FString>::IsVariableString() const
{
	return true;
}


// specialization for bool

template<> TConsoleVariableData<bool>* FConsoleVariable<bool>::AsVariableBool()
{
	return &Data;
}

// specialization for int32

template<> TConsoleVariableData<int32>* FConsoleVariable<int32>::AsVariableInt()
{
	return &Data; 
}

// specialization for float

template<> TConsoleVariableData<float>* FConsoleVariable<float>::AsVariableFloat()
{
	return &Data;
}

// specialization for FString

template<> void FConsoleVariable<FString>::Set(const TCHAR* Value, const FConsoleVariableBase::FResolvedContext& Context)
{
	if (PreprocessSet(Value, Context.Flags, Context.Tag))
	{
		Data.ShadowedValue[0] = Value;
		OnChanged(Context.Flags);
	}
}

template<> TConsoleVariableData<FString>* FConsoleVariable<FString>::AsVariableString()
{
	return &Data;
}

/**
 * A thin console variable that delegates all reads to a Getter and all writes to a Setter.
 * No history, no priority tracking, no platform value caching - the external source is authoritative.
 * Inherits from FConsoleVariableBase for flags, help text, and change callbacks only.
 */
template <class T>
class FDelegatedConsoleVariable : public FConsoleVariableBase
{
public:
	FDelegatedConsoleVariable(
		T InDefaultValue,
		TFunction<T()> InGetter,
		TFunction<void(const T&)> InSetter,
		const TCHAR* Help,
		EConsoleVariableFlags InFlags)
		: FConsoleVariableBase(Help, InFlags)
		, DefaultValue(MoveTemp(InDefaultValue))
		, Getter(MoveTemp(InGetter))
		, Setter(MoveTemp(InSetter))
	{
		check(Getter);
		check(Setter);
		check(!(InFlags & ECVF_RenderThreadSafe)); // Not supported - no shadow data
	}

	virtual void Release() override { delete this; }

	virtual FResolvedContext ResolveContext(const FSetContext& Context) override
	{
		// No history system - just pass through the flags and tag as-is
		return FResolvedContext{ Context.Flags, Context.Tag };
	}

	virtual void Set(const TCHAR* Value, const FResolvedContext& Context) override
	{
		if (CanChange(Context.Flags))
		{
			T ParsedValue;
			UE::ConsoleManager::Private::GetValueFromString<T>(ParsedValue, Value);
			Setter(ParsedValue);
			OnChanged(Context.Flags, false);
		}
	}

	virtual void Unset(EConsoleVariableFlags SetBy, FName Tag) override
	{
		// The getter is always authoritative, Unset has no effect
	}

	virtual bool GetBool() const override   { return FConsoleVariableConversionHelper<T>::GetBool(Getter()); }
	virtual int32 GetInt() const override   { return FConsoleVariableConversionHelper<T>::GetInt(Getter()); }
	virtual float GetFloat() const override { return FConsoleVariableConversionHelper<T>::GetFloat(Getter()); }
	virtual FString GetString() const override { return FConsoleVariableConversionHelper<T>::GetString(Getter()); }

	virtual FString GetDefaultValue() override
	{
		return TTypeToString<T>::ToString(DefaultValue);
	}

	// History not applicable
	virtual void LogHistory(FOutputDevice& Ar) override {}
	virtual SIZE_T GetHistorySize() override { return 0; }

#if ALLOW_OTHER_PLATFORM_CONFIG
	virtual TSharedPtr<IConsoleVariable> GetPlatformValueVariable(FName PlatformName, const FString& DeviceProfileName) override { return nullptr; }
	virtual bool HasPlatformValueVariable(FName PlatformName, const FString& DeviceProfileName) override { return false; }
	virtual void SetOtherPlatformValue(const TCHAR* InValue, EConsoleVariableFlags SetBy, FName Tag) override {}
#endif

private:
	T DefaultValue;
	TFunction<T()> Getter;
	TFunction<void(const T&)> Setter;
};

template<>
void FDelegatedConsoleVariable<FString>::Set(
	const TCHAR* Value,
	const FResolvedContext& Context)
{
	if (CanChange(Context.Flags))
	{
		Setter(FString(Value));
		OnChanged(Context.Flags, false);
	}
}

template <class T>
IConsoleVariable* FConsoleManager::RegisterDelegatedConsoleVariableImpl(
	const TCHAR* Name, T DefaultValue,
	TFunction<T()> Getter, TFunction<void(const T&)> Setter,
	const TCHAR* Help, uint32 Flags)
{
	return AddConsoleObject(
		Name,
		new FDelegatedConsoleVariable<T>(
			MoveTemp(DefaultValue),
			MoveTemp(Getter),
			MoveTemp(Setter),
			Help,
			(EConsoleVariableFlags)Flags))->AsVariable();
}

IConsoleVariable* FConsoleManager::RegisterDelegatedConsoleVariable(const TCHAR* Name, int32 DefaultValue, TFunction<int32()> Getter, TFunction<void(const int32&)> Setter, const TCHAR* Help, uint32 Flags)
{
	return RegisterDelegatedConsoleVariableImpl(Name, DefaultValue, MoveTemp(Getter), MoveTemp(Setter), Help, Flags);
}

IConsoleVariable* FConsoleManager::RegisterDelegatedConsoleVariable(const TCHAR* Name, float DefaultValue, TFunction<float()> Getter, TFunction<void(const float&)> Setter, const TCHAR* Help, uint32 Flags)
{
	return RegisterDelegatedConsoleVariableImpl(Name, DefaultValue, MoveTemp(Getter), MoveTemp(Setter), Help, Flags);
}

IConsoleVariable* FConsoleManager::RegisterDelegatedConsoleVariable(const TCHAR* Name, bool DefaultValue, TFunction<bool()> Getter, TFunction<void(const bool&)> Setter, const TCHAR* Help, uint32 Flags)
{
	return RegisterDelegatedConsoleVariableImpl(Name, DefaultValue, MoveTemp(Getter), MoveTemp(Setter), Help, Flags);
}

IConsoleVariable* FConsoleManager::RegisterDelegatedConsoleVariable(const TCHAR* Name, const FString& DefaultValue, TFunction<FString()> Getter, TFunction<void(const FString&)> Setter, const TCHAR* Help, uint32 Flags)
{
	return RegisterDelegatedConsoleVariableImpl(Name, DefaultValue, MoveTemp(Getter), MoveTemp(Setter), Help, Flags);
}

template<class T>
FConsoleVariableExtendedData<T>::FConsoleVariableExtendedData(const T& DefaultValue, const TCHAR* Help, EConsoleVariableFlags Flags)
	: FConsoleVariableBase(Help, Flags)
{

}

#if ALLOW_OTHER_PLATFORM_CONFIG

/**
 * Helper for FindOrCreatePlatformCVar that will either find the other-platform FConsoleVariable, or create an appropriate one and put it into the
 * cached set of other-platform CVars inside the passed in CVar
 */
template<typename T>
FConsoleVariable<T>* FindOrCreateTypedPlatformCVar(FConsoleVariableExtendedData<T>* CVar, FName PlatformKey)
{
	FRWScopeLock Lock(CVar->PlatformValuesLock, SLT_Write);
	TSharedPtr<IConsoleVariable> PlatformCVar = CVar->PlatformValues.FindRef(PlatformKey);
	if (!PlatformCVar.IsValid())
	{
		PlatformCVar = TSharedPtr<IConsoleVariable>(new FConsoleVariable(CVar->GetDefaultTypedValue(), TEXT("Platform CVar copy"), (EConsoleVariableFlags)(CVar->GetFlags() & ~ECVF_SetByMask), CVar));
		
		// cache it
		CVar->PlatformValues.Add(PlatformKey, PlatformCVar);

		// let the manager know, in case it had any cached platform values that were read in before the main cvar was created
		GetManager().OnCreatedPlatformCVar(CVar, PlatformCVar, PlatformKey);
	}

	return static_cast<FConsoleVariable<T>*>(PlatformCVar.Get());
}

/**
 * Will find an existing other-platform FConsoleVariable, or create one based on the type of variable that was passed in
 */
FConsoleVariableBase* FindOrCreatePlatformCVar(IConsoleVariable* CVar, FName PlatformKey)
{
	if (CVar->IsVariableBool())		return FindOrCreateTypedPlatformCVar(static_cast<FConsoleVariableExtendedData<bool>*>(CVar), PlatformKey);
	if (CVar->IsVariableInt())		return FindOrCreateTypedPlatformCVar(static_cast<FConsoleVariableExtendedData<int32>*>(CVar), PlatformKey);
	if (CVar->IsVariableFloat())	return FindOrCreateTypedPlatformCVar(static_cast<FConsoleVariableExtendedData<float>*>(CVar), PlatformKey);
	if (CVar->IsVariableString())	return FindOrCreateTypedPlatformCVar(static_cast<FConsoleVariableExtendedData<FString>*>(CVar), PlatformKey);

	unimplemented();
	return nullptr;
}

static const FString GSpecialDPNameForPremadePlatformKey("/");
/**
 * Helper function to make a single key used in the PlatformValues set, that combines Platform and a DP name. If the DP name is empty, this will use
 * Platform name as the DP name. This matches up with how GetPlatformValueVariable() was generally used in the past - get the CVar using the platform-named
 * DeviceProfile. 
 * If DeviceProfileName is the GSpecialDPNameForPremadePlatformKey, then that indicates PlatformName is already a PlatformKey that
 * was created with this function before, so just return the PlatformName untouched. This makes is so we don't need two versions of functions
 * that take a PlatformName and DeviceProfileName, and we don't need to keep re-creating PlatformKeys
 */
static FName MakePlatformKey(FName PlatformName, const FString& DeviceProfileName)
{
	// a bit of a hack to say PlatformName is already a Key
	if (DeviceProfileName == GSpecialDPNameForPremadePlatformKey || PlatformName == NAME_None)
	{
		return PlatformName;
	}
	return *(PlatformName.ToString() + TEXT("/") + (DeviceProfileName.Len() ? DeviceProfileName : PlatformName.ToString()));
}

static FString CheckForRemappedDP(FName PlatformName, const FString& DeviceProfileName)
{
	// check if we have an mapped override value (when cooking, usually) unless we passed in a DPName
	if (DeviceProfileName == TEXT(""))
	{
		FName MappedOverride = ConsoleVariablePlatformMapping::GetPlatformToDeviceProfileMapping(PlatformName);
		if (MappedOverride != NAME_None)
		{
			return MappedOverride.ToString();
		}
	}

	return DeviceProfileName;
}

template<class T>
bool FConsoleVariableExtendedData<T>::HasPlatformValueVariable(FName PlatformName, const FString& InDeviceProfileName)
{
	FString DeviceProfileName = CheckForRemappedDP(PlatformName, InDeviceProfileName);

	// cheap lock here, contention is very rare
	FRWScopeLock Lock(PlatformValuesLock, SLT_Write);
	return PlatformValues.Contains(MakePlatformKey(PlatformName, DeviceProfileName));
}

template<class T>
TSharedPtr<IConsoleVariable> FConsoleVariableExtendedData<T>::GetPlatformValueVariable(FName PlatformName, const FString& InDeviceProfileName)
{
	FString DeviceProfileName = CheckForRemappedDP(PlatformName, InDeviceProfileName);

	// if we have GSpecialDPNameForPremadePlatformKey passed in, we have already gone through the
	// Load and we have a premade key in PlatformName
	if (DeviceProfileName != GSpecialDPNameForPremadePlatformKey)
	{
		// make sure we have cached this platform/DP
		IConsoleManager::Get().LoadAllPlatformCVars(PlatformName, DeviceProfileName);
	}

	// we have assumed in the past that we would return at least the constructor version, so create one if we are explicitly asking
	// this can happen when .ini files don't give a value to a cvar, but we are asking for a platform's value anyway - in
	// which case we want the constructor value, not the current platform's value
	if (!HasPlatformValueVariable(PlatformName, DeviceProfileName))
	{
		FindOrCreatePlatformCVar(this, MakePlatformKey(PlatformName, DeviceProfileName));
	}
	
	// cheap lock here, contention is very rare
	FRWScopeLock Lock(PlatformValuesLock, SLT_Write);
	return PlatformValues.FindRef(MakePlatformKey(PlatformName, DeviceProfileName));
}

template<class T>
void FConsoleVariableExtendedData<T>::ClearPlatformVariables(FName PlatformName)
{
	FRWScopeLock Lock(PlatformValuesLock, SLT_Write);

	if (PlatformName == NAME_None)
	{
		PlatformValues.Empty();
	}
	else
	{
		PlatformValues.Remove(PlatformName);
	}
}

#endif



// ----

// T: int32, float, bool
template <class T>
class FConsoleVariableRef : public FConsoleVariableExtendedData<T>
{
	// help find functions without needing this-> prefixes
	using FConsoleVariableBase::GetShadowIndex;
	using FConsoleVariableBase::CanChange;
	using FConsoleVariableBase::Flags;

public:
	FConsoleVariableRef(T& InRefValue, const TCHAR* Help, EConsoleVariableFlags Flags) 
		: FConsoleVariableExtendedData<T>(InRefValue, Help, Flags)
		, RefValue(InRefValue)
		, MainValue(InRefValue)
	{
	}

	// interface IConsoleVariable -----------------------------------

	virtual void Release()
	{
		delete this; 
	}

	virtual void Set(const TCHAR* Value, const FConsoleVariableBase::FResolvedContext& Context) override
	{
		if (this->TrackHistory(Value, Context.Flags, Context.Tag) == false)
		{
			return;
		}
		
		if (CanChange(Context.Flags))
		{
			UE::ConsoleManager::Private::GetValueFromString<T>(MainValue, Value);
			OnChanged(Context.Flags);
		}
	}

	virtual bool IsVariableBool() const override { return false; }
	virtual bool IsVariableInt() const override { return false; }
	virtual bool IsVariableFloat() const override { return false; }
	virtual bool IsVariableString() const override { return false; }

	virtual bool GetBool() const override { return FConsoleVariableConversionHelper<T>::GetBool(Value()); }
	virtual int32 GetInt() const override { return FConsoleVariableConversionHelper<T>::GetInt(Value()); }
	virtual float GetFloat() const override { return FConsoleVariableConversionHelper<T>::GetFloat(Value()); }
	virtual FString GetString() const override { return FConsoleVariableConversionHelper<T>::GetString(Value()); }

private: // ----------------------------------------------------

	// reference the the value (should not be changed from outside), if ECVF_RenderThreadSafe this is the render thread version, otherwise same as MainValue
	T& RefValue;
	//  main thread version 
	T MainValue;
	
	const T &Value() const
	{
		uint32 Index = GetShadowIndex();
		checkSlow(Index < 2);
		return (Index == 0) ? MainValue : RefValue;
	}
	
	virtual void SetInternal(const T& TypedValue, EConsoleVariableFlags SetBy)
	{
		MainValue = TypedValue;
	}

	void OnChanged(EConsoleVariableFlags SetBy, bool bForce=false)
	{
		// propagate from main thread to render thread or to reference
		OnCVarChange(RefValue, MainValue, Flags, SetBy);
		FConsoleVariableBase::OnChanged(SetBy, bForce);
	}
};

template <>
bool FConsoleVariableRef<bool>::IsVariableBool() const
{
	return true;
}

template <>
bool FConsoleVariableRef<int32>::IsVariableInt() const
{
	return true;
}

template <>
bool FConsoleVariableRef<float>::IsVariableFloat() const
{
	return true;
}


// string version

class FConsoleVariableStringRef : public FConsoleVariableExtendedData<FString>
{
public:
	FConsoleVariableStringRef(FString& InRefValue, const TCHAR* Help, EConsoleVariableFlags Flags)
		: FConsoleVariableExtendedData<FString>(FString(), Help, Flags)
		, RefValue(InRefValue)
		, MainValue(InRefValue)
	{
	}

	// interface IConsoleVariable -----------------------------------

	virtual void Release()
	{
		delete this;
	}
	virtual void Set(const TCHAR* Value, const FConsoleVariableBase::FResolvedContext& Context) override
	{
		if (PreprocessSet(Value, Context.Flags, Context.Tag))
		{
			MainValue = Value;
			OnChanged(Context.Flags);
		}
	}
	virtual bool GetBool() const
	{
		bool Result = false;
		TTypeFromString<bool>::FromString(Result, *MainValue);
		return Result;
	}
	virtual int32 GetInt() const
	{
		int32 Result = 0;
		TTypeFromString<int32>::FromString(Result, *MainValue);
		return Result;
	}
	virtual float GetFloat() const
	{
		float Result = 0.0f;
		UE::ConsoleManager::Private::GetValueFromString<float>(Result, *MainValue);
		return Result;
	}
	virtual FString GetString() const
	{
		return MainValue;
	}
	virtual bool IsVariableString() const override
	{
		return true;
	}

private: // ----------------------------------------------------

	// reference the the value (should not be changed from outside), if ECVF_RenderThreadSafe this is the render thread version, otherwise same as MainValue
	FString& RefValue;
	// main thread version 
	FString MainValue;

	const FString& Value() const
	{
		uint32 Index = GetShadowIndex();
		checkSlow(Index < 2);
		return (Index == 0) ? MainValue : RefValue;
	}

	virtual void SetInternal(const FString& TypedValue, EConsoleVariableFlags SetBy)
	{
		MainValue = TypedValue;
	}
	
	void OnChanged(EConsoleVariableFlags SetBy, bool bForce=false)
	{
		// propagate from main thread to render thread or to reference
		OnCVarChange(RefValue, MainValue, Flags, SetBy);
		FConsoleVariableBase::OnChanged(SetBy, bForce);
	}
};

// name version

class FConsoleVariableNameRef : public FConsoleVariableExtendedData<FString>
{
public:
	FConsoleVariableNameRef(FName& InRefValue, const TCHAR* Help, EConsoleVariableFlags Flags)
	: FConsoleVariableExtendedData<FString>(FString(), Help, Flags)
	, RefValue(InRefValue)
	, MainValue(InRefValue)
	{
	}

	// interface IConsoleVariable -----------------------------------

	virtual void Release()
	{
		delete this;
	}
	virtual void Set(const TCHAR* Value, const FConsoleVariableBase::FResolvedContext& Context) override
	{
		TrackHistory(Value, Context.Flags, Context.Tag);

		if (CanChange(Context.Flags))
		{
			MainValue = FName(Value);
			OnChanged(Context.Flags);
		}
	}
	virtual bool GetBool() const
	{
		ensureMsgf(false, TEXT("Getting the bool value of a FName based CVar is not supported"));
		return false;
	}
	virtual int32 GetInt() const
	{
		ensureMsgf(false, TEXT("Getting the integer value of a FName based CVar is not supported"));
		return 0;
	}
	virtual float GetFloat() const
	{
		ensureMsgf(false, TEXT("Getting the float value of a FName based CVar is not supported"));
		return 0.0f;
	}
	virtual FString GetString() const
	{
		return MainValue.ToString();
	}
	virtual bool IsVariableString() const override
	{
		return true;
	}

private: // ----------------------------------------------------

	// reference the the value (should not be changed from outside), if ECVF_RenderThreadSafe this is the render thread version, otherwise same as MainValue
	FName& RefValue;
	// main thread version 
	FName MainValue;

	const FName& Value() const
	{
		uint32 Index = GetShadowIndex();
		checkSlow(Index < 2);
		return (Index == 0) ? MainValue : RefValue;
	}

	virtual void SetInternal(const FString& TypedValue, EConsoleVariableFlags SetBy) override
	{
		MainValue = FName(TypedValue);
	}
	
	void OnChanged(EConsoleVariableFlags SetBy, bool bForce=false)
	{
		// propagate from main thread to render thread or to reference
		OnCVarChange(RefValue, MainValue, Flags, SetBy);
		FConsoleVariableBase::OnChanged(SetBy, bForce);
	}
};


class FConsoleVariableBitRef : public FConsoleVariableExtendedData<int>
{
public:
	FConsoleVariableBitRef(const TCHAR* FlagName, uint32 InBitNumber, uint8* InForce0MaskPtr, uint8* InForce1MaskPtr, const TCHAR* Help, EConsoleVariableFlags Flags) 
		: FConsoleVariableExtendedData<int>(0, Help, Flags), Force0MaskPtr(InForce0MaskPtr), Force1MaskPtr(InForce1MaskPtr), BitNumber(InBitNumber)
	{
	}

	// interface IConsoleVariable -----------------------------------

	virtual void Release()
	{
		delete this; 
	} 
	virtual void Set(const TCHAR* InValue, const FConsoleVariableBase::FResolvedContext& Context) override
	{
		if (PreprocessSet(InValue, Context.Flags, Context.Tag))
		{
			int32 Value = FCString::Atoi(InValue);

			check(IsInGameThread());

			FMath::SetBoolInBitField(Force0MaskPtr, BitNumber, Value == 0);
			FMath::SetBoolInBitField(Force1MaskPtr, BitNumber, Value == 1);

			OnChanged(Context.Flags, false);
		}
	}
	virtual bool GetBool() const
	{
		return GetInt() != 0;
	}
	virtual int32 GetInt() const
	{
		// we apply the bitmask on game thread (showflags) so we don't have to do any special thread handling
		check(IsInGameThread());

		bool Force0 = FMath::ExtractBoolFromBitfield(Force0MaskPtr, BitNumber);
		bool Force1 = FMath::ExtractBoolFromBitfield(Force1MaskPtr, BitNumber);
		
		if(!Force0 && !Force1)
		{
			// not enforced to be 0 or 1
			return 2;
		}

		return Force1 ? 1 : 0;
	}
	virtual float GetFloat() const
	{
		return (float)GetInt();
	}
	virtual FString GetString() const
	{
		return FString::Printf(TEXT("%d"), GetInt());
	}

private: // ----------------------------------------------------

	uint8* Force0MaskPtr;
	uint8* Force1MaskPtr;
	uint32 BitNumber;
	
	virtual void SetInternal(const int& TypedValue, EConsoleVariableFlags SetBy)
	{
		FMath::SetBoolInBitField(Force0MaskPtr, BitNumber, TypedValue == 0);
		FMath::SetBoolInBitField(Force1MaskPtr, BitNumber, TypedValue == 1);
	}
	
};

IConsoleVariable* FConsoleManager::RegisterConsoleVariableBitRef(const TCHAR* CVarName, const TCHAR* FlagName, uint32 BitNumber, uint8* Force0MaskPtr, uint8* Force1MaskPtr, const TCHAR* Help, uint32 Flags)
{
	return AddConsoleObject(CVarName, new FConsoleVariableBitRef(FlagName, BitNumber, Force0MaskPtr, Force1MaskPtr, Help, (EConsoleVariableFlags)Flags))->AsVariable();
}


template<typename T>
class FRangedConsoleVariable : public FConsoleVariable<T>
{
	using Base = FConsoleVariable<T>;
public:
	IConsoleVariable* Min;
	IConsoleVariable* Max;

	FRangedConsoleVariable(const FString& Name, T DefaultValue, T MinValue, T MaxValue, const TCHAR* Help, EConsoleVariableFlags Flags, IConsoleVariable* Parent)
		: FConsoleVariable<T>(DefaultValue, Help, Flags, Parent)
	{
		FConsoleManager& Manager = (FConsoleManager&)IConsoleManager::Get();
		Min = Manager.RegisterConsoleVariable(*(Name + ".min"), MinValue, *FString::Printf(TEXT("The min value for CVar %s"), *Name), Flags);
		Max = Manager.RegisterConsoleVariable(*(Name + ".max"), MaxValue, *FString::Printf(TEXT("The max value for CVar %s"), *Name), Flags);

		// when min and max change, trigger an update of this cvar so any listeners can deal with _potential_ new value returned from Get
		Min->SetOnChangedCallback(FConsoleVariableDelegate::CreateLambda([this](IConsoleVariable*) { this->OnChangedDelegate().Broadcast(this); }));
		Max->SetOnChangedCallback(FConsoleVariableDelegate::CreateLambda([this](IConsoleVariable*) { this->OnChangedDelegate().Broadcast(this); }));
	}

	virtual bool GetBool() const override
	{
		unimplemented();
		return false;
	}
	virtual int32 GetInt() const override
	{
		return FMath::Clamp(Base::GetInt(), Min->GetInt(), Max->GetInt());
	}
	virtual float GetFloat() const override
	{
		return FMath::Clamp(Base::GetFloat(), Min->GetFloat(), Max->GetFloat());
	}
	virtual FString GetString() const override
	{
		T Value, MinValue, MaxValue;
		this->GetValue(Value);
		Min->GetValue(MinValue);
		Max->GetValue(MaxValue);

		return LexToString(FMath::Clamp(Value, MinValue, MaxValue));
	}
};

template<>
TAutoRangedConsoleVariable<int32>::TAutoRangedConsoleVariable(IConsoleVariable* Parent, const TCHAR* Name, int32 DefaultValue, int32 MinValue, int32 MaxValue, const TCHAR* Help, uint32 Flags)
	: TAutoConsoleVariable(GetManager().AddConsoleObject(Name, new FRangedConsoleVariable(Name, DefaultValue, MinValue, MaxValue, Help, (EConsoleVariableFlags)Flags, Parent))->AsVariable())
{
}

template<>
TAutoRangedConsoleVariable<float>::TAutoRangedConsoleVariable(IConsoleVariable* Parent, const TCHAR* Name, float DefaultValue, float MinValue, float MaxValue, const TCHAR* Help, uint32 Flags)
	: TAutoConsoleVariable(GetManager().AddConsoleObject(Name, new FRangedConsoleVariable(Name, DefaultValue, MinValue, MaxValue, Help, (EConsoleVariableFlags)Flags, Parent))->AsVariable())
{
}

void FConsoleManager::CallAllConsoleVariableSinks()
{
	QUICK_SCOPE_CYCLE_COUNTER(ConsoleManager_CallAllConsoleVariableSinks);

	if (bCallAllConsoleVariableSinks)
	{
		for (int32 i = 0; i < ConsoleVariableChangeSinks.Num(); ++i)
		{
			ConsoleVariableChangeSinks[i].ExecuteIfBound();
		}

		bCallAllConsoleVariableSinks = false;
	}
}

FConsoleVariableSinkHandle FConsoleManager::RegisterConsoleVariableSink_Handle(const FConsoleCommandDelegate& Command)
{
	ConsoleVariableChangeSinks.Add(Command);
	return FConsoleVariableSinkHandle(Command.GetHandle());
}

void FConsoleManager::UnregisterConsoleVariableSink_Handle(FConsoleVariableSinkHandle Handle)
{
	ConsoleVariableChangeSinks.RemoveAll([=](const FConsoleCommandDelegate& Delegate){ return Handle.HasSameHandle(Delegate); });
}

class FConsoleCommand : public FConsoleCommandBase
{

public:
	FConsoleCommand( const FConsoleCommandDelegate& InitDelegate, const TCHAR* InitHelp, const EConsoleVariableFlags InitFlags )
		: FConsoleCommandBase( InitHelp, InitFlags ),
		  Delegate( InitDelegate )
	{
	}

	// interface IConsoleCommand -----------------------------------

	virtual void Release() override
	{
		delete this; 
	} 

	virtual bool Execute( const TArray< FString >& Args, UWorld* InWorld, FOutputDevice& OutputDevice ) override
	{
		// NOTE: Args are ignored for FConsoleCommand.  Use FConsoleCommandWithArgs if you need parameters.
		return Delegate.ExecuteIfBound();
	}

private:

	/** User function to call when the console command is executed */
	FConsoleCommandDelegate Delegate;

};


class FConsoleCommandWithArgs : public FConsoleCommandBase
{

public:
	FConsoleCommandWithArgs( const FConsoleCommandWithArgsDelegate& InitDelegate, const TCHAR* InitHelp, const EConsoleVariableFlags InitFlags )
		: FConsoleCommandBase( InitHelp, InitFlags ),
		  Delegate( InitDelegate )
	{
	}

	// interface IConsoleCommand -----------------------------------

	virtual void Release() override
	{
		delete this; 
	} 

	virtual bool Execute( const TArray< FString >& Args, UWorld* InWorld, FOutputDevice& OutputDevice ) override
	{
		return Delegate.ExecuteIfBound( Args );
	}

private:

	/** User function to call when the console command is executed */
	FConsoleCommandWithArgsDelegate Delegate;
};

/* Console command that can be given a world parameter. */
class FConsoleCommandWithWorld : public FConsoleCommandBase
{

public:
	FConsoleCommandWithWorld( const FConsoleCommandWithWorldDelegate& InitDelegate, const TCHAR* InitHelp, const EConsoleVariableFlags InitFlags )
		: FConsoleCommandBase( InitHelp, InitFlags ),
		Delegate( InitDelegate )
	{
	}

	// interface IConsoleCommand -----------------------------------

	virtual void Release() override
	{
		delete this; 
	} 

	virtual bool Execute( const TArray< FString >& Args, UWorld* InWorld, FOutputDevice& OutputDevice ) override
	{
		return Delegate.ExecuteIfBound( InWorld );
	}

private:

	/** User function to call when the console command is executed */
	FConsoleCommandWithWorldDelegate Delegate;
};

/* Console command that can be given a world parameter and args. */
class FConsoleCommandWithWorldAndArgs : public FConsoleCommandBase
{

public:
	FConsoleCommandWithWorldAndArgs( const FConsoleCommandWithWorldAndArgsDelegate& InitDelegate, const TCHAR* InitHelp, const EConsoleVariableFlags InitFlags )
		: FConsoleCommandBase( InitHelp, InitFlags ),
		Delegate( InitDelegate )
	{
	}

	// interface IConsoleCommand -----------------------------------

	virtual void Release() override
	{
		delete this; 
	} 

	virtual bool Execute( const TArray< FString >& Args, UWorld* InWorld, FOutputDevice& OutputDevice ) override
	{
		return Delegate.ExecuteIfBound( Args, InWorld );
	}

private:

	/** User function to call when the console command is executed */
	FConsoleCommandWithWorldAndArgsDelegate Delegate;
};

/* Console command that can be given args and an output device. */
class FConsoleCommanWithArgsAndOutputDevice : public FConsoleCommandBase
{

public:
	FConsoleCommanWithArgsAndOutputDevice(const FConsoleCommandWithArgsAndOutputDeviceDelegate& InitDelegate, const TCHAR* InitHelp, const EConsoleVariableFlags InitFlags)
		: FConsoleCommandBase(InitHelp, InitFlags),
		Delegate(InitDelegate)
	{
	}

	// interface IConsoleCommand -----------------------------------

	virtual void Release() override
	{
		delete this;
	}

	virtual bool Execute(const TArray< FString >& Args, UWorld* InWorld, FOutputDevice& OutputDevice) override
	{
		return Delegate.ExecuteIfBound(Args, OutputDevice);
	}

private:

	/** User function to call when the console command is executed */
	FConsoleCommandWithArgsAndOutputDeviceDelegate Delegate;
};

/* Console command that can be given a world parameter, args and an output device. */
class FConsoleCommandWithWorldArgsAndOutputDevice : public FConsoleCommandBase
{

public:
	FConsoleCommandWithWorldArgsAndOutputDevice(const FConsoleCommandWithWorldArgsAndOutputDeviceDelegate& InitDelegate, const TCHAR* InitHelp, const EConsoleVariableFlags InitFlags)
		: FConsoleCommandBase(InitHelp, InitFlags),
		Delegate(InitDelegate)
	{
	}

	// interface IConsoleCommand -----------------------------------

	virtual void Release() override
	{
		delete this;
	}

	virtual bool Execute(const TArray< FString >& Args, UWorld* InWorld, FOutputDevice& OutputDevice) override
	{
		return Delegate.ExecuteIfBound(Args, InWorld, OutputDevice);
	}

private:

	/** User function to call when the console command is executed */
	FConsoleCommandWithWorldArgsAndOutputDeviceDelegate Delegate;
};


/* Console command that can be given an output device. */
class FConsoleCommandWithOutputDevice : public FConsoleCommandBase
{

public:
	FConsoleCommandWithOutputDevice( const FConsoleCommandWithOutputDeviceDelegate& InitDelegate, const TCHAR* InitHelp, const EConsoleVariableFlags InitFlags )
		: FConsoleCommandBase( InitHelp, InitFlags ),
		Delegate( InitDelegate )
	{
	}

	// interface IConsoleCommand -----------------------------------

	virtual void Release() override
	{
		delete this; 
	} 

	virtual bool Execute( const TArray< FString >& Args, UWorld* InWorld, FOutputDevice& OutputDevice ) override
	{
		return Delegate.ExecuteIfBound( OutputDevice );
	}

private:

	/** User function to call when the console command is executed */
	FConsoleCommandWithOutputDeviceDelegate Delegate;
};

// only needed for auto completion of Exec commands
class FConsoleCommandExec : public FConsoleCommandBase
{

public:
	FConsoleCommandExec( const TCHAR* InitHelp, const EConsoleVariableFlags InitFlags )
		: FConsoleCommandBase( InitHelp, InitFlags )
	{
	}

	// interface IConsoleCommand -----------------------------------

	virtual void Release() override
	{
		delete this; 
	} 

	virtual bool Execute( const TArray< FString >& Args, UWorld* InCmdWorld, FOutputDevice& OutputDevice ) override
	{
		return false;
	}
};


FConsoleManager::FConsoleManager()
	: bHistoryWasLoaded(false)
	, ThreadPropagationCallback(0)
	, bCallAllConsoleVariableSinks(true)
{}

FConsoleManager::~FConsoleManager()
{
	for (TMap<FString, IConsoleObject*>::TConstIterator PairIt(ConsoleObjects); PairIt; ++PairIt)
	{
		IConsoleObject* Var = PairIt.Value();
		delete Var;
	}
}

IConsoleVariable* FConsoleManager::RegisterConsoleVariable(const TCHAR* Name, bool DefaultValue, const TCHAR* Help, uint32 Flags)
{
	LLM_SCOPE_BYNAME(TEXT("EngineMisc/ConsoleCommands"));
	return AddConsoleObject(Name, new FConsoleVariable<bool>(DefaultValue, Help, (EConsoleVariableFlags)Flags))->AsVariable();
}

IConsoleVariable* FConsoleManager::RegisterConsoleVariable(const TCHAR* Name, int32 DefaultValue, const TCHAR* Help, uint32 Flags)
{
	LLM_SCOPE_BYNAME(TEXT("EngineMisc/ConsoleCommands"));
	return AddConsoleObject(Name, new FConsoleVariable<int32>(DefaultValue, Help, (EConsoleVariableFlags)Flags))->AsVariable();
}

IConsoleVariable* FConsoleManager::RegisterConsoleVariable(const TCHAR* Name, float DefaultValue, const TCHAR* Help, uint32 Flags)
{
	LLM_SCOPE_BYNAME(TEXT("EngineMisc/ConsoleCommands"));
	return AddConsoleObject(Name, new FConsoleVariable<float>(DefaultValue, Help, (EConsoleVariableFlags)Flags))->AsVariable();
}

IConsoleVariable* FConsoleManager::RegisterConsoleVariable(const TCHAR* Name, const TCHAR* DefaultValue, const TCHAR* Help, uint32 Flags)
{
	LLM_SCOPE_BYNAME(TEXT("EngineMisc/ConsoleCommands"));
	return RegisterConsoleVariable(Name, FString(DefaultValue), Help, Flags);
}

IConsoleVariable* FConsoleManager::RegisterConsoleVariable(const TCHAR* Name, const FString& DefaultValue, const TCHAR* Help, uint32 Flags)
{
	// not supported
	check((Flags & (uint32)ECVF_RenderThreadSafe) == 0);
	LLM_SCOPE_BYNAME(TEXT("EngineMisc/ConsoleCommands"));
	return AddConsoleObject(Name, new FConsoleVariable<FString>(DefaultValue, Help, (EConsoleVariableFlags)Flags))->AsVariable();
}

IConsoleVariable* FConsoleManager::RegisterConsoleVariableRef(const TCHAR* Name, bool& RefValue, const TCHAR* Help, uint32 Flags)
{
	LLM_SCOPE_BYNAME(TEXT("EngineMisc/ConsoleCommands"));
	return AddConsoleObject(Name, new FConsoleVariableRef<bool>(RefValue, Help, (EConsoleVariableFlags)Flags))->AsVariable();
}

IConsoleVariable* FConsoleManager::RegisterConsoleVariableRef(const TCHAR* Name, int32& RefValue, const TCHAR* Help, uint32 Flags)
{
	LLM_SCOPE_BYNAME(TEXT("EngineMisc/ConsoleCommands"));
	return AddConsoleObject(Name, new FConsoleVariableRef<int32>(RefValue, Help, (EConsoleVariableFlags)Flags))->AsVariable();
}

IConsoleVariable* FConsoleManager::RegisterConsoleVariableRef(const TCHAR* Name, float& RefValue, const TCHAR* Help, uint32 Flags)
{
	LLM_SCOPE_BYNAME(TEXT("EngineMisc/ConsoleCommands"));
	return AddConsoleObject(Name, new FConsoleVariableRef<float>(RefValue, Help, (EConsoleVariableFlags)Flags))->AsVariable();
}

IConsoleVariable* FConsoleManager::RegisterConsoleVariableRef(const TCHAR* Name, FString& RefValue, const TCHAR* Help, uint32 Flags)
{
	LLM_SCOPE_BYNAME(TEXT("EngineMisc/ConsoleCommands"));
	return AddConsoleObject(Name, new FConsoleVariableStringRef(RefValue, Help, (EConsoleVariableFlags)Flags))->AsVariable();
}

IConsoleVariable* FConsoleManager::RegisterConsoleVariableRef(const TCHAR* Name, FName& RefValue, const TCHAR* Help, uint32 Flags)
{
	LLM_SCOPE_BYNAME(TEXT("EngineMisc/ConsoleCommands"));
	return AddConsoleObject(Name, new FConsoleVariableNameRef(RefValue, Help, (EConsoleVariableFlags)Flags))->AsVariable();
}


IConsoleCommand* FConsoleManager::RegisterConsoleCommand(const TCHAR* Name, const TCHAR* Help, const FConsoleCommandDelegate& Command, uint32 Flags)
{
	LLM_SCOPE_BYNAME(TEXT("EngineMisc/ConsoleCommands"));
	return AddConsoleObject(Name, new FConsoleCommand(Command, Help, (EConsoleVariableFlags)Flags))->AsCommand();
}

IConsoleCommand* FConsoleManager::RegisterConsoleCommand(const TCHAR* Name, const TCHAR* Help, uint32 Flags)
{
	LLM_SCOPE_BYNAME(TEXT("EngineMisc/ConsoleCommands"));
	return AddConsoleObject(Name, new FConsoleCommandExec(Help, (EConsoleVariableFlags)Flags))->AsCommand();
}

IConsoleCommand* FConsoleManager::RegisterConsoleCommand(const TCHAR* Name, const TCHAR* Help, const FConsoleCommandWithArgsDelegate& Command, uint32 Flags)
{
	LLM_SCOPE_BYNAME(TEXT("EngineMisc/ConsoleCommands"));
	return AddConsoleObject(Name, new FConsoleCommandWithArgs(Command, Help, (EConsoleVariableFlags)Flags))->AsCommand();
}

IConsoleCommand* FConsoleManager::RegisterConsoleCommand(const TCHAR* Name, const TCHAR* Help, const FConsoleCommandWithWorldDelegate& Command, uint32 Flags)
{
	LLM_SCOPE_BYNAME(TEXT("EngineMisc/ConsoleCommands"));
	return AddConsoleObject(Name, new FConsoleCommandWithWorld(Command, Help, (EConsoleVariableFlags)Flags))->AsCommand();
}

IConsoleCommand* FConsoleManager::RegisterConsoleCommand(const TCHAR* Name, const TCHAR* Help, const FConsoleCommandWithWorldAndArgsDelegate& Command, uint32 Flags)
{
	LLM_SCOPE_BYNAME(TEXT("EngineMisc/ConsoleCommands"));
	return AddConsoleObject(Name, new FConsoleCommandWithWorldAndArgs(Command, Help, (EConsoleVariableFlags)Flags))->AsCommand();
}

IConsoleCommand* FConsoleManager::RegisterConsoleCommand(const TCHAR* Name, const TCHAR* Help, const FConsoleCommandWithArgsAndOutputDeviceDelegate& Command, uint32 Flags)
{
	LLM_SCOPE_BYNAME(TEXT("EngineMisc/ConsoleCommands"));
	return AddConsoleObject(Name, new FConsoleCommanWithArgsAndOutputDevice(Command, Help, (EConsoleVariableFlags)Flags))->AsCommand();
}

IConsoleCommand* FConsoleManager::RegisterConsoleCommand(const TCHAR* Name, const TCHAR* Help, const FConsoleCommandWithWorldArgsAndOutputDeviceDelegate& Command, uint32 Flags)
{
	LLM_SCOPE_BYNAME(TEXT("EngineMisc/ConsoleCommands"));
	return AddConsoleObject(Name, new FConsoleCommandWithWorldArgsAndOutputDevice(Command, Help, (EConsoleVariableFlags)Flags))->AsCommand();
}

IConsoleCommand* FConsoleManager::RegisterConsoleCommand(const TCHAR* Name, const TCHAR* Help, const FConsoleCommandWithOutputDeviceDelegate& Command, uint32 Flags)
{
	LLM_SCOPE_BYNAME(TEXT("EngineMisc/ConsoleCommands"));
	return AddConsoleObject(Name, new FConsoleCommandWithOutputDevice(Command, Help, (EConsoleVariableFlags)Flags))->AsCommand();
}


IConsoleVariable* FConsoleManager::FindConsoleVariable(const TCHAR* Name, bool bTrackFrequentCalls) const
{
	IConsoleObject* Obj = FindConsoleObject(Name, bTrackFrequentCalls);

	if(Obj)
	{
		if(Obj->TestFlags(ECVF_Unregistered))
		{
			return 0;
		}

		return Obj->AsVariable();
	}

	return 0;
}

IConsoleObject* FConsoleManager::FindConsoleObject(const TCHAR* Name, bool bTrackFrequentCalls) const
{
	IConsoleObject* CVar = FindConsoleObjectUnfiltered(Name);

#if TRACK_CONSOLE_FIND_COUNT
	if (bTrackFrequentCalls)
	{
		const bool bEarlyAppPhase = GFrameCounter < 1000;
		if(CVar)
		{
			++CVar->FindCallCount;

			// we test for equal to avoid log spam
			if(bEarlyAppPhase && CVar->FindCallCount == 500)
			{
				UE_LOGF(LogConsoleManager, Warning, "Performance warning: Console object named '%ls' shows many (%d) FindConsoleObject() calls (consider caching e.g. using static)", Name, CVar->FindCallCount);
			}
		}
		else// if (bEarlyAppPhase || GFrameCounter == 1000)
		{
			static uint32 NullFindCallCount = 0;
			static TMap<FName, uint32> PerNameNullFindCallCount;

			++NullFindCallCount;
			FName CVarName(Name);
			if (PerNameNullFindCallCount.FindOrAdd(Name)++ == 30)
			{
				UE_LOGF(LogConsoleManager, Warning, "Performance warning: Many (%d) failed FindConsoleObject() for '%ls'. ", PerNameNullFindCallCount[Name], Name);
			}

			if(NullFindCallCount == 500)
			{
				UE_LOGF(LogConsoleManager, Warning, "Performance warning: Many (%d) failed FindConsoleObject() across all CVars. Fail counts per name:", NullFindCallCount);
				for (TPair<FName, uint32> Entry : PerNameNullFindCallCount)
				{
					UE_LOGF(LogConsoleManager, Warning, "   %ls : %d", *Entry.Key.ToString(), Entry.Value);
				}
			}
		}
	}
#endif

	if(CVar && CVar->TestFlags(ECVF_CreatedFromIni))
	{
		return nullptr;
	}

	return CVar;
}

IConsoleObject* FConsoleManager::FindConsoleObjectUnfiltered(const TCHAR* Name) const
{
	UE::TScopeLock ScopeLock(ConsoleObjectsSynchronizationObject);
	IConsoleObject* Var = ConsoleObjects.FindRef(Name);
	return Var;
}

void FConsoleManager::UnregisterConsoleObject(IConsoleObject* CVar, bool bKeepState)
{
	if(!CVar)
	{
		return;
	}
	UE::TScopeLock ScopeLock(ConsoleObjectsSynchronizationObject);

	// Slow search for console object
	const FString ObjName = FindConsoleObjectName( CVar );
	if( !ObjName.IsEmpty() )
	{
		UnregisterConsoleObject(*ObjName, bKeepState);
	}
}


void FConsoleManager::UnregisterConsoleObject(const TCHAR* Name, bool bKeepState)
{
	UE::TScopeLock ScopeLock(ConsoleObjectsSynchronizationObject);

	IConsoleObject* Object = FindConsoleObject(Name);

	if(Object)
	{
		ConsoleObjectUnregisteredDelegate.Broadcast(Name, Object);

		IConsoleVariable* CVar = Object->AsVariable();

		if (CVar)
		{
			ConsoleVariableUnregisteredDelegate.Broadcast(CVar);
		}

		if(CVar && bKeepState)
		{
			// to be able to restore the value if we just recompile a module
			CVar->SetFlags(ECVF_Unregistered);
		}
		else
		{
			ConsoleObjects.Remove(Name);
			Object->Release();
		}
	}
}

void FConsoleManager::LoadHistoryIfNeeded()
{
	if (bHistoryWasLoaded)
	{
		return;
	}

	bHistoryWasLoaded = true;
	HistoryEntriesMap.Reset();

	FConfigFile Ini;
	FConfigContext::ReadSingleIntoLocalFile(Ini).Load(*(FPaths::GeneratedConfigDir() + TEXT("ConsoleHistory.ini")));

	const FString SectionName = "ConsoleHistory";
	const FName KeyName = "History";

	for (const auto& ConfigPair : AsConst(Ini))
	{
		FString HistoryKey;
		if (ConfigPair.Key == SectionName)
		{
			// uses empty HistoryKey
		}
		else if (ConfigPair.Key.StartsWith(SectionName))
		{
			HistoryKey = ConfigPair.Key.Mid(SectionName.Len());
		}
		else
		{
			continue;
		}

		TArray<FString>& HistoryEntries = HistoryEntriesMap.FindOrAdd(HistoryKey);
		for (const auto& ConfigSectionPair : ConfigPair.Value)
		{
			if (ConfigSectionPair.Key == KeyName)
			{
				HistoryEntries.Add(ConfigSectionPair.Value.GetValue());
			}
		}
	}
}

void FConsoleManager::SaveHistory()
{
	FConfigFile Ini;
	
	const FString SectionName = "ConsoleHistory";
	const FName KeyName = "History";

	for (const auto& HistoryPair : HistoryEntriesMap)
	{
		FString ConfigSectionName = FString::Printf(TEXT("%s%s"), *SectionName, *HistoryPair.Key);
		for (const auto& HistoryEntry : HistoryPair.Value)
		{
			Ini.AddToSection(*ConfigSectionName, KeyName, HistoryEntry);
		}
	}

	const FString ConfigPath = FPaths::GeneratedConfigDir() + TEXT("ConsoleHistory.ini");

	Ini.Dirty = true;
	Ini.Write(ConfigPath);
}

void FConsoleManager::ForEachConsoleObjectThatStartsWith(const FConsoleObjectVisitor& Visitor, const TCHAR* ThatStartsWith) const
{
	check(Visitor.IsBound());
	check(ThatStartsWith);

	//@caution, potential deadlock if the visitor tries to call back into the cvar system. Best not to do this, but we could capture and array of them, then release the lock, then dispatch the visitor.
	UE::TScopeLock ScopeLock(ConsoleObjectsSynchronizationObject);
	for(TMap<FString, IConsoleObject*>::TConstIterator PairIt(ConsoleObjects); PairIt; ++PairIt)
	{
		const FString& Name = PairIt.Key();
		IConsoleObject* CVar = PairIt.Value();

		if(!CVar->IsShadowObject() && MatchPartialName(*Name, ThatStartsWith))
		{
			Visitor.Execute(*Name, CVar);
		}
	}
}

void FConsoleManager::ForEachConsoleObjectThatContains(const FConsoleObjectVisitor& Visitor, const TCHAR* ThatContains) const
{
	check(Visitor.IsBound());
	check(ThatContains);

	TArray<FString> ThatContainsArray;
	FString(ThatContains).ParseIntoArray(ThatContainsArray, TEXT(" "), true);
	int32 ContainsStringLength = FCString::Strlen(ThatContains);

	//@caution, potential deadlock if the visitor tries to call back into the cvar system. Best not to do this, but we could capture and array of them, then release the lock, then dispatch the visitor.
	UE::TScopeLock ScopeLock(ConsoleObjectsSynchronizationObject);
	for(TMap<FString, IConsoleObject*>::TConstIterator PairIt(ConsoleObjects); PairIt; ++PairIt)
	{
		const FString& Name = PairIt.Key();
		IConsoleObject* CVar = PairIt.Value();

		if (CVar->IsShadowObject())
		{
			continue;
		}
		if (ContainsStringLength == 1)
		{
			if (MatchPartialName(*Name, ThatContains))
			{
				Visitor.Execute(*Name, CVar);
			}
		}
		else
		{
			bool bMatchesAll = true;

			for (int32 MatchIndex = 0; MatchIndex < ThatContainsArray.Num(); MatchIndex++)
			{
				if (!MatchSubstring(*Name, *ThatContainsArray[MatchIndex]))
				{
					bMatchesAll = false;
				}
			}

			if (bMatchesAll && ThatContainsArray.Num() > 0)
			{
				Visitor.Execute(*Name, CVar);
			}
		}
	}
}

template <typename FmtType, typename... Types>
static void MultiLogf(FOutputDevice* Device, FArchive* File, const FmtType& Fmt, Types... Args)
{
	if (Device != nullptr)
	{
		Device->Logf(Fmt, Args...);
	}
	if (File != nullptr)
	{
		File->Logf(Fmt, Args...);
	}
}

void FConsoleManager::DumpObjects(const TCHAR* Params, FOutputDevice& InAr, bool bDisplayCommands) const
{
	bool bShowHelp = FParse::Param(Params, TEXT("showhelp"));
	bool bShowDeprecated = FParse::Param(Params, TEXT("deprecated"));

	FString CSVFilename;
	bool bWriteToCSV = FParse::Value(Params, TEXT("-csv="), CSVFilename);
	bWriteToCSV = bWriteToCSV || FParse::Param(Params, TEXT("csv"));
	FString FilterSetBy;
	FParse::Value(Params, TEXT("-setby="), FilterSetBy);
	FString Prefix = FParse::Token(Params, false);
	if (Prefix.StartsWith(TEXT("-")))
	{
		Prefix = TEXT("");
	}

	// get sorted list of keys of all console objects
	TArray<FString> SortedKeys;
	ConsoleObjects.GetKeys(SortedKeys);
	SortedKeys.Sort();

	FOutputDevice* Log = nullptr;
	FArchive* CSV = nullptr;
	if (bWriteToCSV)
	{
		if (CSVFilename.IsEmpty())
		{
			CSVFilename = FPaths::Combine(FPaths::ProjectLogDir(), bDisplayCommands ? TEXT("ConsoleCommands.csv") : TEXT("ConsoleVars.csv"));
		}
		CSV = IFileManager::Get().CreateFileWriter(*CSVFilename, FILEWRITE_AllowRead);
		if (CSV == nullptr)
		{
			InAr.Logf(TEXT("Unable to create CSV file for writing: '%s'"), *CSVFilename);
			return;
		}

		InAr.Logf(TEXT("Dumping to CSV file: '%s'"), *CSVFilename);
		if (bDisplayCommands)
		{
			CSV->Logf(TEXT("NAME%s"), bShowHelp ? TEXT(",HELP") : TEXT(""));
		}
		else
		{
			CSV->Logf(TEXT("NAME,VALUE,SETBY%s"), bShowHelp ? TEXT(",HELP") : TEXT(""));
		}
	}
	else
	{
		// only write to the log if CSV is not used
		Log = &InAr;
	}

	for (const FString& Key : SortedKeys)
	{
		if (Prefix.IsEmpty() || Key.StartsWith(Prefix))
		{
			IConsoleObject* Obj = ConsoleObjects[Key];
			IConsoleVariable* CVar = Obj->AsVariable();
			IConsoleCommand* CCmd = Obj->AsCommand();

			if (Obj->TestFlags(ECVF_Unregistered))
			{
				continue;
			}
			
			bool bDisplay = true;
			bDisplay &= bShowDeprecated == Obj->IsDeprecated();
			bDisplay &= bDisplayCommands ? CCmd != nullptr : CVar != nullptr;
			if (!bDisplay)
			{
				continue;
			}

			FString Annotation =
				Obj->IsDeprecated() ? TEXT(" [DEPRECATED]") :
				Obj->IsShadowObject() ? TEXT(" [SHADOW]") :
				TEXT("");

			// process optional help
			FString Help;
			if (bShowHelp)
			{
				Help = FString(Obj->GetHelp()).TrimStartAndEnd();
				if (bWriteToCSV)
				{
					// newlines and commas in help will throw off the csv
					Help = FString::Printf(TEXT(",\"%s\""), *Help.Replace(TEXT("\n"), TEXT("\\n")));
				}
				else
				{
					Help = FString::Printf(TEXT("\n%s\n "), *Help);
				}
			}

			if (CCmd != nullptr)
			{
				MultiLogf(Log, CSV, TEXT("%s%s%s"), *Key, *Annotation, *Help);
			}
			if (CVar != nullptr)
			{
				const TCHAR* SetBy = GetConsoleVariableSetByName(CVar->GetFlags());
				if (FilterSetBy.Len() > 0 && SetBy != FilterSetBy)
				{
					continue;
				}

				// Don't read the value when deprecated. It will trip the UsageBehavior, which may assert.
				FString Value = Obj->IsDeprecated() ? TEXT("<deprecated, can't show>") : CVar->GetString();

				if (bWriteToCSV)
				{
					MultiLogf(Log, CSV, TEXT("%s%s,%s,%s%s"), *Key, *Annotation, *Value, SetBy, *Help);
				}
				else
				{
					MultiLogf(Log, CSV, TEXT("%s%s = \"%s\"      LastSetBy: %s%s"), *Key, *Annotation, *Value, SetBy, *Help);
				}
			}
		}
	}

	delete CSV;
}


#if ALLOW_OTHER_PLATFORM_CONFIG
static inline FString MakeUnknownCVarKey(const FString& CVarName, FName PlatformKey)
{
	return FString::Printf(TEXT("%s@%s"), *CVarName, *PlatformKey.ToString());
}

void FConsoleManager::OnCreatedPlatformCVar(IConsoleVariable* MainVariable, TSharedPtr<IConsoleVariable> PlatformVariable, FName PlatformKey)
{
	FString UnknownCVarKey = MakeUnknownCVarKey(FindConsoleObjectName(MainVariable), PlatformKey);

	FScopeLock Lock(&CachedPlatformsAndDeviceProfilesLock);

	// look to see if there are any cached off values loaded from disk but created before the cvar was created
	TArray<TTuple<FString, EConsoleVariableFlags>*> LoadedValues;
	UnknownCVarCache.MultiFindPointer(UnknownCVarKey, LoadedValues, true);
	if (LoadedValues.Num() > 0)
	{
		for (TTuple<FString, EConsoleVariableFlags>* Value : LoadedValues)
		{
			PlatformVariable->Set(*Value->Key, Value->Value);

			UE_LOGF(LogConsoleManager, Verbose, "Setting delayed platform cvar value, %ls = %ls, at priority %ls", *UnknownCVarKey, *Value->Key, GetConsoleVariableSetByName(Value->Value));
		}

		UnknownCVarCache.Remove(UnknownCVarKey);
	}
}
#endif

static void SetUnsetCVar(const TMap<FString, IConsoleObject*>& ConsoleObjects, const TCHAR* Params, FOutputDevice& Ar, bool bSet)
{
	FString CVarName = FParse::Token(Params, false);

	if (CVarName.Len() == 0)
	{
		if (bSet)
		{
			Ar.Logf(TEXT("Usage: SetCVar [Platform@]CVarName Value [-setby=Priority] [-tag=SomeTag] [-replacecurrent]"));
		}
		else
		{
			Ar.Logf(TEXT("Usage: UnsetCVar [Platform@]CVarName [-setby=Priority] [-tag=SomeTag]"));
		}
		Ar.Logf(TEXT("   Priority can be one of the following (default is Console):"));
		
		#define LOGOP(x) Ar.Logf(TEXT("      %s%s"), TEXT(#x), IsArrayPriority(ECVF_SetBy##x) ? TEXT(" [*]") : TEXT(""));
		ENUMERATE_SET_BY(LOGOP)
		#undef LOGOP
		Ar.Logf(TEXT("      [*] Array type priorities, used for dynamic setting/unsetting"));
		Ar.Logf(TEXT("   Tag should be set for the ones marked as Array types, for ability to set and unset"));

	}
	
	FString PlatformName;
	FString DeviceProfileName;
	int32 PlatformDelim = CVarName.Find(TEXT("@"));
	if (PlatformDelim > 0)
	{
		PlatformName = CVarName.Mid(0, PlatformDelim);
		if (PlatformName.Contains(TEXT("/")))
		{
			FString Plat;
			PlatformName.Split(TEXT("/"), &Plat, &DeviceProfileName);
			PlatformName = *Plat;
		}

		CVarName = CVarName.Mid(PlatformDelim + 1);
	}

	IConsoleObject* Obj = ConsoleObjects.FindRef(CVarName);
	IConsoleVariable* CVar = Obj ? Obj->AsVariable() : nullptr;
	
	if (CVar == nullptr)
	{
		Ar.Logf(TEXT("No CVar named %s"), *CVarName);
		return;
	}
	
	// get platform version
	if (PlatformName.Len())
	{
#if ALLOW_OTHER_PLATFORM_CONFIG
		CVar = CVar->GetPlatformValueVariable(*PlatformName, *DeviceProfileName).Get();
		if (CVar == nullptr)
		{
			Ar.Logf(TEXT("Failed to get CVar for platform %s"), *PlatformName);
			return;
		}
#else
		Ar.Logf(TEXT("Unable to lookup a CVar value on another platform in this build"));
		return;
#endif
	}
	
	FString Value;
	if (bSet)
	{
		Value = FParse::Token(Params, false);
	}

	if (bSet && FParse::Param(Params, TEXT("-replacecurrent")))
	{
		CVar->ReplaceCurrentPriorityAndTag(*Value);
	}
	else
	{
		EConsoleVariableFlags SetBy = ECVF_SetByConsole;
		FName Tag = NAME_None;
		FString Str;
		if (FParse::Value(Params, TEXT("-setby="), Str))
		{
			SetBy = GetConsoleVariableSetByValue(*Str);
		}
		if (FParse::Value(Params, TEXT("-tag="), Str))
		{
			Tag = *Str;
		}

		if (bSet)
		{
			CVar->Set(*Value, SetBy, Tag);
		}
		else
		{
			CVar->Unset(SetBy, Tag);
		}
	}
}

void UnsetCVarTag(const TCHAR* Params, FOutputDevice& Ar)
{
	FString TagName = FParse::Token(Params, false);
	IConsoleManager::Get().UnsetAllConsoleVariablesWithTag(*TagName);
}

UE_TRACE_EVENT_BEGIN(Cpu, ConsoleCmd, NoSync)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, Cmd)
UE_TRACE_EVENT_END()

bool FConsoleManager::ProcessUserConsoleInput(const TCHAR* InInput, FOutputDevice& Ar, UWorld* InWorld)
{
	check(InInput);

#if CPUPROFILERTRACE_ENABLED
	UE_TRACE_LOG_SCOPED_T(Cpu, ConsoleCmd, CpuChannel)
		<< ConsoleCmd.Cmd(InInput);
#endif

	CSV_EVENT_GLOBAL(TEXT("Cmd: %s"), InInput);

	if (FParse::Command(&InInput, TEXT("dumpcvars")))
	{
		DumpObjects(InInput, Ar, false);
		return true;
	}
	if (FParse::Command(&InInput, TEXT("dumpccmds")))
	{
		DumpObjects(InInput, Ar, true);
		return true;
	}
	if (FParse::Command(&InInput, TEXT("setcvar")))
	{
		SetUnsetCVar(ConsoleObjects, InInput, Ar, true);
		return true;
	}
	if (FParse::Command(&InInput, TEXT("unsetcvar")))
	{
		SetUnsetCVar(ConsoleObjects, InInput, Ar, false);
		return true;
	}
	if (FParse::Command(&InInput, TEXT("unsetcvartag")))
	{
		UnsetCVarTag(InInput, Ar);
		return true;
	}

	const TCHAR* It = InInput;

	FString Param1 = GetTextSection(It);
	if(Param1.IsEmpty())
	{
		return false;
	}

	// Remove a trailing ? if present, to kick it into help mode
	const bool bCommandEndedInQuestion = Param1.EndsWith(TEXT("?"), ESearchCase::CaseSensitive);
	if (bCommandEndedInQuestion)
	{
		Param1.MidInline(0, Param1.Len() - 1, EAllowShrinking::No);
	}

	// look for the <cvar>@<platform[/deviceprofile]> syntax
	FName PlatformName;
	FString DeviceProfileName;
	if (Param1.Contains(TEXT("@")))
	{
		FString Left, Right;
		Param1.Split(TEXT("@"), &Left, &Right);

		if (Left.Len() && Right.Len())
		{
			Param1 = Left;
			if (Right.Contains(TEXT("/")))
			{
				FString Plat;
				Right.Split(TEXT("/"), &Plat, &DeviceProfileName);
				PlatformName = *Plat;
			}
			else
			{
				PlatformName = *Right;
			}
		}
	}

	IConsoleObject* CObj = FindConsoleObject(*Param1);
	if(!CObj)
	{
		return false;
	}

	if (!CObj->IsEnabled())
	{
		return false;
	}

	// fix case for nicer printout
	Param1 = FindConsoleObjectName(CObj);

	IConsoleCommand* CCmd = CObj->AsCommand();
	IConsoleVariable* CVar = CObj->AsVariable();
	TSharedPtr<IConsoleVariable> PlatformCVar;

	if (PlatformName != NAME_None)
	{
		if (CVar == nullptr)
		{
			Ar.Logf(TEXT("Ignoring platform portion (@%s), which is only valid for looking up CVars"), *PlatformName.ToString());
		}
		else
		{
#if ALLOW_OTHER_PLATFORM_CONFIG
			PlatformCVar = CVar->GetPlatformValueVariable(PlatformName, DeviceProfileName);
			CVar = PlatformCVar.Get();
			if (!CVar)
			{
				Ar.Logf(TEXT("Unable find CVar %s for platform %s (possibly invalid platform name?)"), *Param1, *PlatformName.ToString());
				return false;
			}
#else
			Ar.Logf(TEXT("Unable to lookup a CVar value on another platform in this build"));
			return false;
#endif
		}
	}

	if( CCmd )
	{
		// Process command
		// Build up argument list
		TArray<FString> Args;
		FString Arg;
		while (FParse::Token(It, Arg, false))
		{
			Args.Add(Arg);
		}

		const bool bShowHelp = bCommandEndedInQuestion || ((Args.Num() == 1) && (Args[0] == TEXT("?")));
		if( bShowHelp )
		{
			// get help
			Ar.Logf(TEXT("HELP for '%s':\n%s"), *Param1, CCmd->GetHelp());
		}
		else
		{
			// if a delegate was bound, we execute it and it should return true,
			// otherwise it was a Exec console command and this returns FASLE
			return CCmd->Execute( Args, InWorld, Ar );
		}
	}
	else if( CVar )
	{
		// Process variable
		bool bShowHelp = bCommandEndedInQuestion;
		bool bShowCurrentState = false;

		if(*It == 0)
		{
			bShowCurrentState = true;
		}
		else
		{
			FString Param2 = FString(It).TrimStartAndEnd();

			const bool bReadOnly = CVar->TestFlags(ECVF_ReadOnly);

			if(Param2.Len() >= 2)
			{
				if(Param2[0] == (TCHAR)'\"' && Param2[Param2.Len() - 1] == (TCHAR)'\"')
				{
					Param2.MidInline(1, Param2.Len() - 2, EAllowShrinking::No);
				}
				// this is assumed to be unintended e.g. copy and paste accident from ini file
				if(Param2.Len() > 0 && Param2[0] == (TCHAR)'=')
				{
					Ar.Logf(TEXT("Warning: Processing the console input parameters the leading '=' is ignored (only needed for ini files)."));
					Param2.MidInline(1, Param2.Len() - 1, EAllowShrinking::No);
				}
			}

			if (Param2 == TEXT("?"))
			{
				bShowHelp = true;
			}
			else
			{
				if (PlatformName != NAME_None)
				{
					Ar.Logf(TEXT("Error: Unable to set a value for %s another platform!"), *Param1);
				}
				else if(bReadOnly)
				{
					Ar.Logf(TEXT("Error: %s is read only!"), *Param1);
				}
				else
				{
					// set value
					CVar->Set(*Param2, ECVF_SetByConsole);

					Ar.Logf(TEXT("%s = \"%s\""), *Param1, *CVar->GetString());
				}
			}
		}

		if(bShowHelp)
		{
			// get help
			const bool bReadOnly = CVar->TestFlags(ECVF_ReadOnly);
			Ar.Logf(TEXT("HELP for '%s'%s:\n%s"), *Param1, bReadOnly ? TEXT("(ReadOnly)") : TEXT(""), CVar->GetHelp());
			bShowCurrentState = true;
		}

		if(bShowCurrentState)
		{
			((FConsoleVariableBase*)CVar)->LogHistory(Ar);

			Ar.Logf(TEXT("%s = \"%s\"      LastSetBy: %s"), *Param1, *CVar->GetString(), GetConsoleVariableSetByName(CVar->GetFlags()));
		}
	}

	return true;
}

void FConsoleManager::AddShadowConsoleObject(const TCHAR* Name, IConsoleObject* Obj)
{
	ConsoleObjects.Add(Name, Obj);
}

IConsoleObject* FConsoleManager::AddConsoleObject(const TCHAR* Name, IConsoleObject* Obj)
{
	check(Name);
	check(*Name != 0);
	check(Obj);
	LLM_SCOPE_BYNAME(TEXT("EngineMisc/ConsoleCommands"));

	UE::TScopeLock ScopeLock(ConsoleObjectsSynchronizationObject); // we will lock on the entire add process
	IConsoleObject* ExistingObj = ConsoleObjects.FindRef(Name);

	if(Obj->GetFlags() & ECVF_Scalability)
	{
		// Scalability options cannot be cheats - otherwise using the options menu would mean cheating
		check(!(Obj->GetFlags() & ECVF_Cheat));
		// Scalability options cannot be read only - otherwise the options menu cannot work 
		check(!(Obj->GetFlags() & ECVF_ReadOnly));
	}

	if(Obj->GetFlags() & ECVF_RenderThreadSafe)
	{
		if(Obj->AsCommand())
		{
			// This feature is not supported for console commands
			check(0);
		}
	}

#if UE_EDITOR
	if (Obj->GetFlags() & ECVF_SaveForNextBoot)
	{
		UE::ConfigUtilities::ApplyCVarOfDllFromBootHotfix(Name, Obj);
	}
#endif

	if(ExistingObj)
	{
		// An existing console object was found that has the same name as the object being registered.
		// In most cases this is not allowed, but if there is a variable with the same name and is
		// in an 'unregistered' state or we're hot reloading dlls, we may be able to replace or update that variable.
#if WITH_RELOAD
		const bool bCanUpdateOrReplaceObj = (ExistingObj->AsVariable()||ExistingObj->AsCommand()) && (IsReloadActive() || ExistingObj->TestFlags(ECVF_Unregistered));
#else
		const bool bCanUpdateOrReplaceObj = ExistingObj->AsVariable() && ExistingObj->TestFlags(ECVF_Unregistered);
#endif
		if( !bCanUpdateOrReplaceObj )
		{
			// NOTE: The reason we don't assert here is because when using HotReload, locally-initialized static console variables will be
			//       re-registered, and it's desirable for the new variables to clobber the old ones.  Because this happen outside of the
			//       reload stack frame (IsActiveReload()=true), we can't detect and handle only those cases, so we opt to warn instead.
			UE_LOGF(LogConsoleManager, Warning, "Console object named '%ls' already exists but is being registered again, but we weren't expected it to be! (FConsoleManager::AddConsoleObject)", Name );
		}

		IConsoleVariable* ExistingVar = ExistingObj->AsVariable();
		IConsoleCommand* ExistingCmd = ExistingObj->AsCommand();
		const int ExistingType = ExistingVar ? ExistingCmd ? 3 : 2 : 1;

		IConsoleVariable* Var = Obj->AsVariable();
		IConsoleCommand* Cmd = Obj->AsCommand();
		const int NewType = Var ? Cmd ? 3 : 2 : 1;

		// Validate that we have the same type for the existing console object and for the new one, because
		// never allowed to replace a command with a variable or vice-versa
		if( ExistingType != NewType )
		{
			UE_LOGF(LogConsoleManager, Fatal, "Console object named '%ls' can't be replaced with the new one of different type!", Name );
		}

		if( ExistingVar && Var )
		{
			if(ExistingVar->TestFlags(ECVF_CreatedFromIni))
			{
				// Allow the scalability system to update its own values during initialization.
				bool bScalabilityUpdate = (Var->GetFlags() & ECVF_SetByScalability) && (ExistingVar->GetFlags() & ECVF_SetByScalability);

				// This is to prevent cheaters to set a value from an ini of a cvar that is created later
				// TODO: This is not ideal as it also prevents consolevariables.ini to set the value where we allow that. We could fix that.
				if(!Var->TestFlags(ECVF_Cheat) && !bScalabilityUpdate)
				{
					// The existing one came from the ini, get the value
					Var->Set(*ExistingVar->GetString(), (EConsoleVariableFlags)((uint32)ExistingVar->GetFlags() & ECVF_SetByMask));
				}

				// destroy the existing one (no need to call sink because that will happen after all ini setting have been loaded)
				ExistingVar->Release();

				ConsoleObjects.Add(Name, Var);
				return Var;
			}
#if WITH_RELOAD
			else if (IsReloadActive())
			{
				// Variable is being replaced due to a hot reload - copy state across to new variable, but only if the type hasn't changed
				{
					if (ExistingVar->IsVariableFloat())
					{
						Var->Set(ExistingVar->GetFloat());
					}
				}
				{
					if (ExistingVar->IsVariableInt())
					{
						Var->Set(ExistingVar->GetInt());
					}
				}
				{
					if (ExistingVar->IsVariableString())
					{
						Var->Set(*ExistingVar->GetString());
					}
				}
				ExistingVar->Release();
				ConsoleObjects.Add(Name, Var);
				return Var;
			}
#endif
			else
			{
				// Copy data over from the new variable,
				// but keep the value from the existing one.
				// This way references to the old variable are preserved (no crash).
				// Changing the type of a variable however is not possible with this.
				ExistingVar->SetFlags(Var->GetFlags());
				ExistingVar->SetHelp(Var->GetHelp());

				// Name was already registered but got unregistered
				Var->Release();

				return ExistingVar;
			}
		}
		else if( ExistingCmd )
		{
			// Replace console command with the new one and release the existing one.
			// This should be safe, because we don't have FindConsoleVariable equivalent for commands.
			ConsoleObjects.Add( Name, Cmd );
			ExistingCmd->Release();

			return Cmd;
		}

		// Should never happen
		return nullptr;
	}
	else
	{
		ConsoleObjects.Add(Name, Obj);
		return Obj;
	}
}

FString FConsoleManager::GetTextSection(const TCHAR* &It)
{
	FString ret;

	while(*It)
	{
		if(IsWhiteSpace(*It))
		{
			break;
		}

		ret += *It++;
	}

	while(IsWhiteSpace(*It))
	{
		++It;
	}

	return ret;
}

FString FConsoleManager::FindConsoleObjectName(const IConsoleObject* InVar) const
{
	check(InVar);

	UE::TScopeLock ScopeLock(ConsoleObjectsSynchronizationObject);
	for(TMap<FString, IConsoleObject*>::TConstIterator PairIt(ConsoleObjects); PairIt; ++PairIt)
	{
		IConsoleObject* Var = PairIt.Value();

		if(Var == InVar)
		{
			const FString& Name = PairIt.Key();

			return Name;
		}
	}

	// if we didn't find one, and it has a parent, then give that a try
	if (InVar->GetParentObject() != nullptr)
	{
		return FindConsoleObjectName(InVar->GetParentObject());
	}

	return FString();
}

bool FConsoleManager::MatchPartialName(const TCHAR* Stream, const TCHAR* Pattern)
{
	while(*Pattern)
	{
		if(FChar::ToLower(*Stream) != FChar::ToLower(*Pattern))
		{
			return false;
		}

		++Stream;
		++Pattern;
	}

	return true;
}

bool FConsoleManager::MatchSubstring(const TCHAR* Stream, const TCHAR* Pattern)
{
	while(*Stream)
	{
		int32 StreamIndex = 0;
		int32 PatternIndex = 0;

		do
		{
			if (Pattern[PatternIndex] == 0)
			{
				return true;
			}
			else if (FChar::ToLower(Stream[StreamIndex]) != FChar::ToLower(Pattern[PatternIndex]))
			{
				break;
			}

			PatternIndex++;
			StreamIndex++;
		} 
		while (Stream[StreamIndex] != 0 || Pattern[PatternIndex] == 0);

		++Stream;
	}

	return false;
}

void CreateConsoleVariables();

IConsoleManager* IConsoleManager::Singleton;

void IConsoleManager::SetupSingleton()
{
	check(!Singleton);
	LLM_SCOPE_BYNAME(TEXT("EngineMisc/ConsoleCommands"));
	if (!Singleton)
	{
		Singleton = new FConsoleManager; // we will leak this
		CreateConsoleVariables();
	}
	check(Singleton);
}

void FConsoleManager::AddConsoleHistoryEntry(const TCHAR* Key, const TCHAR* Input)
{
	LoadHistoryIfNeeded();

	TArray<FString>& HistoryEntries = HistoryEntriesMap.FindOrAdd(Key);

	// limit size to avoid a ever growing file
	while(HistoryEntries.Num() > 64)
	{
		HistoryEntries.RemoveAt(0);
	}

	const FString InString(Input);
	HistoryEntries.Remove(InString);
	HistoryEntries.Add(InString);

	SaveHistory();
}

void FConsoleManager::GetConsoleHistory(const TCHAR* Key, TArray<FString>& Out)
{
	LoadHistoryIfNeeded();

	Out = HistoryEntriesMap.FindOrAdd(Key);
}

bool FConsoleManager::IsNameRegistered(const TCHAR* Name) const
{
	UE::TScopeLock ScopeLock(ConsoleObjectsSynchronizationObject);
	return ConsoleObjects.Contains(Name);
}

void FConsoleManager::RegisterThreadPropagation(uint32 ThreadId, IConsoleThreadPropagation* InCallback)
{
	if(InCallback)
	{
		// at the moment we only support one thread besides the main thread
		check(!ThreadPropagationCallback);
	}
	else
	{
		// bad input parameters
		check(!ThreadId);
	}

	ThreadPropagationCallback = InCallback;
	// `ThreadId` is ignored as only RenderingThread is supported
}

IConsoleThreadPropagation* FConsoleManager::GetThreadPropagationCallback()
{
	return ThreadPropagationCallback;
}

bool FConsoleManager::IsThreadPropagationThread()
{
	return IsInActualRenderingThread();
}

void FConsoleManager::OnCVarChanged()
{
	bCallAllConsoleVariableSinks = true;
}

FConsoleVariableMulticastDelegate& FConsoleManager::OnCVarUnregistered()
{
	return ConsoleVariableUnregisteredDelegate;
}

FConsoleObjectWithNameMulticastDelegate& FConsoleManager::OnConsoleObjectUnregistered()
{
	return ConsoleObjectUnregisteredDelegate;
}

void FConsoleManager::UnsetAllConsoleVariablesWithTag(FName Tag, EConsoleVariableFlags Priority)
{
	TSet<IConsoleVariable*>* TaggedSet = UE::ConsoleManager::Private::TaggedCVars.FindRef(Tag);
	if (TaggedSet == nullptr)
	{
		return;
	}
	
	for (IConsoleVariable* Var : *TaggedSet)
	{
		Var->Unset(Priority, Tag);
	}
	
	delete UE::ConsoleManager::Private::TaggedCVars.FindRef(Tag);
	UE::ConsoleManager::Private::TaggedCVars.Remove(Tag);
}

void FConsoleManager::BatchUpdateTag(FName Tag, const TMap<FName, FString>& CVarsAndValues)
{
	// cache the tagged cvars 
	TSet<IConsoleVariable*>* TaggedCVarsPtr = UE::ConsoleManager::Private::TaggedCVars.FindRef(Tag);
	TSet<IConsoleVariable*> TaggedCVars;
	if (TaggedCVarsPtr != nullptr)
	{
		TaggedCVars = *TaggedCVarsPtr;
	}

	for (auto& Pair : CVarsAndValues)
	{
		FString CVarNameString = Pair.Key.ToString();
		IConsoleObject* CObj = ConsoleObjects.FindRef(CVarNameString);
		// @todo handle when not found to make a dummy one?
		if (CObj)
		{
			if (IConsoleVariable* CVar = CObj->AsVariable())
			{
				// remove this from the set of tracked cvars, so we don't unset it at the end
				TaggedCVars.Remove(CVar);
			}
		}
	}

	// now unset aren't going to be set - this has to be done before we delete the global TaggedCVars below
	// @todo - deal with different SetBys?
	for (IConsoleVariable* CVar : TaggedCVars)
	{
		CVar->Unset(ECVF_SetByMask, Tag);
	}

	// remove it from the set, so we can add back cleanly
	delete UE::ConsoleManager::Private::TaggedCVars.FindRef(Tag);
	UE::ConsoleManager::Private::TaggedCVars.Remove(Tag);

	// now set everything, without reordering the array types
	for (auto& Pair : CVarsAndValues)
	{
		FString CVarNameString = Pair.Key.ToString();
		IConsoleObject* CObj = ConsoleObjects.FindRef(CVarNameString);
		// @todo handle when not found to make a dummy one?
		if (CObj)
		{
			if (IConsoleVariable* CVar = CObj->AsVariable())
			{
				bool bIsScalabilityBucket = CVarNameString.StartsWith(TEXT("sg."));
				uint32 CVarPriority = bIsScalabilityBucket ? ECVF_SetByScalability : ECVF_SetByDeviceProfile;
				// since we are updating, replace with existing key
				CVar->Set(*Pair.Value, (EConsoleVariableFlags)(CVarPriority | ECVF_Set_ReplaceExistingTag), Tag);
			}
		}
	}

}

#if ALLOW_OTHER_PLATFORM_CONFIG

void FConsoleManager::LoadAllPlatformCVars(FName PlatformName, const FString& DeviceProfileName)
{
	UE::ConfigAccessTracking::FIgnoreScope IgnoreScope;

	FName PlatformKey = MakePlatformKey(PlatformName, DeviceProfileName);
	
	// protect the cached CVar info from two threads trying to get a platform CVar at once, and both attempting to load all of the cvars at the same time
	FScopeLock Lock(&CachedPlatformsAndDeviceProfilesLock);
	if (CachedPlatformsAndDeviceProfiles.Contains(PlatformKey))
	{
		return;
	}
	CachedPlatformsAndDeviceProfiles.Add(PlatformKey);
	
	// use the platform's base DeviceProfile for emulation
	VisitPlatformCVarsForEmulation(PlatformName, DeviceProfileName.IsEmpty() ? PlatformName.ToString() : DeviceProfileName,
		[PlatformName, PlatformKey, this](const FString& CVarName, const FString& CVarValue, EConsoleVariableFlags SetByAndPreview)
	{
		int32 SetBy = SetByAndPreview & ECVF_SetByMask;
			
		// make sure the named cvar exists
		IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(*CVarName, false /* bTrackFrequentCalls */);
		if (CVar == nullptr)
		{
			// store this unknown value in a map, so when the cvar gets created later, we can grab this platform's values into the platform variable
			FString UnknownCVarKey = MakeUnknownCVarKey(CVarName, PlatformKey);
			UnknownCVarCache.Add(UnknownCVarKey, { CVarValue, (EConsoleVariableFlags)SetBy });

			UE_LOGF(LogConsoleManager, Verbose, "Loaded %ls@%ls = %ls, but the main CVar doesn't exist!", *CVarName, *PlatformKey.ToString(), *CVarValue);
			return;
		}
		
		// find or make the cvar for this platformkey
		FConsoleVariableBase* PlatformCVar = FindOrCreatePlatformCVar(CVar, PlatformKey);

		// now cache the passed in value
		PlatformCVar->SetOtherPlatformValue(*CVarValue, (EConsoleVariableFlags)SetBy, NAME_None);
		
		UE_LOGF(LogConsoleManager, Verbose, "Loading %ls@%ls = %ls [get = %ls]", *CVarName, *PlatformKey.ToString(),
			   *CVarValue, *CVar->GetPlatformValueVariable(*PlatformName.ToString())->GetString());
		
	});
}

void FConsoleManager::PreviewPlatformCVars(FName PlatformName, const FString& DeviceProfileName, FName PreviewModeTag)
{
	// we want Preview but not Cheat
	StompPlatformCVars(PlatformName, DeviceProfileName, PreviewModeTag, ECVF_SetByPreview, ECVF_Preview, ECVF_Cheat);
}

void FConsoleManager::StompPlatformCVars(FName PlatformName, const FString& DeviceProfileName, FName Tag, EConsoleVariableFlags SetBy, EConsoleVariableFlags RequiredFlags, EConsoleVariableFlags DisallowedFlags)
{
	UE_LOGF(LogConsoleManager, Display, "Previewing/Stomping CVars for Platform '%ls', DeviceProfile '%ls', ModeTag '%ls'", *PlatformName.ToString(), *DeviceProfileName, *Tag.ToString());

	LoadAllPlatformCVars(PlatformName, DeviceProfileName.Len() ? DeviceProfileName : PlatformName.ToString());

	FName PlatformKey = MakePlatformKey(PlatformName, DeviceProfileName);

	for (auto Pair : ConsoleObjects)
	{
		if (Pair.Value->IsShadowObject())
		{
			continue;
		}
		if (IConsoleVariable* CVar = Pair.Value->AsVariable())
		{
			// require some flags, but disallow others
			if ((CVar->GetFlags() & (RequiredFlags | DisallowedFlags)) == RequiredFlags)
			{
				EConsoleVariableFlags Flags = SetBy;
				if (CVar->GetFlags() & ECVF_ScalabilityGroup)
				{
					// we want to set SG cvars so they can be queried, but not send updates so that we don't use host platform's cvars
					Flags = (EConsoleVariableFlags)(Flags | ECVF_Set_SetOnly_Unsafe);
				}

				// if we have a value for the platform, then set it in the real CVar
				if (CVar->HasPlatformValueVariable(PlatformKey, GSpecialDPNameForPremadePlatformKey))
				{
					TSharedPtr<IConsoleVariable> PlatformCVar = CVar->GetPlatformValueVariable(PlatformKey, GSpecialDPNameForPremadePlatformKey);
					CVar->Set(*PlatformCVar->GetString(), Flags, Tag);

					UE_LOGF(LogConsoleManager, Display, "  |-> Setting %ls = %ls", *Pair.Key, *PlatformCVar->GetString());
				}
			}
		}
	}
}

void FConsoleManager::ClearAllPlatformCVars(FName PlatformName, const FString& DeviceProfileName)
{
	FName PlatformKey = MakePlatformKey(PlatformName, DeviceProfileName);

	// protect the cached CVar info from two threads trying to get a platform CVar at once, and both attempting to load all of the cvars at the same time
	FScopeLock Lock(&CachedPlatformsAndDeviceProfilesLock);
	
	if (!CachedPlatformsAndDeviceProfiles.Contains(PlatformKey))
	{
		return;
	}
	CachedPlatformsAndDeviceProfiles.Remove(PlatformKey);
	
	for (auto Pair : ConsoleObjects)
	{
		if (Pair.Value->IsShadowObject())
		{
			continue;
		}

		if (IConsoleVariable* CVar = Pair.Value->AsVariable())
		{
			// clear any cached values for this key
			CVar->ClearPlatformVariables(PlatformKey);
		}
	}
}

#endif

// These don't belong here, but they belong here more than they belong in launch engine loop.
void CreateConsoleVariables()
{
#if !NO_CVARS
	LLM_SCOPE_BYNAME(TEXT("EngineMisc/ConsoleCommands"));
	// this registeres to a reference, so we cannot use TAutoConsoleVariable
	IConsoleManager::Get().RegisterConsoleVariableRef(TEXT("r.DumpingMovie"),
		GIsDumpingMovie,
		TEXT("Allows to dump each rendered frame to disk (slow fps, names MovieFrame..).\n"
			 "<=0:off (default), <0:remains on, >0:remains on for n frames (n is the number specified)"),
		ECVF_Cheat);


#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

	// the following commands are common exec commands that should be added to auto completion (todo: read UnConsole list in ini, discover all exec commands)
	IConsoleManager::Get().RegisterConsoleCommand(TEXT("VisualizeTexture"),	TEXT("To visualize internal textures"), ECVF_Cheat);
	IConsoleManager::Get().RegisterConsoleCommand(TEXT("Vis"),	TEXT("short version of visualizetexture"), ECVF_Cheat);
	IConsoleManager::Get().RegisterConsoleCommand(TEXT("VisRT"),	TEXT("GUI for visualizetexture"), ECVF_Cheat);
	IConsoleManager::Get().RegisterConsoleCommand(TEXT("HighResShot"),	TEXT("High resolution screenshots ResolutionX(int32)xResolutionY(int32) Or Magnification(float) [CaptureRegionX(int32) CaptureRegionY(int32) CaptureRegionWidth(int32) CaptureRegionHeight(int32) MaskEnabled(int32) DumpBufferVisualizationTargets(int32) CaptureHDR(int32)]\nExample: HighResShot 500x500 50 50 120 500 1 1 1"), ECVF_Cheat);
	IConsoleManager::Get().RegisterConsoleCommand(TEXT("DumpUnbuiltLightInteractions"),	TEXT("Logs all lights and primitives that have an unbuilt interaction."), ECVF_Cheat);
	IConsoleManager::Get().RegisterConsoleCommand(TEXT("Stat MapBuildData"),	TEXT(""), ECVF_Cheat);
	IConsoleManager::Get().RegisterConsoleCommand(TEXT("r.ResetViewState"), TEXT("Reset some state (e.g. TemporalAA index) to make rendering more deterministic (for automated screenshot verification)"), ECVF_Cheat);
	IConsoleManager::Get().RegisterConsoleCommand(TEXT("r.RHI.Name"),		TEXT("Show current RHI's name"), ECVF_Cheat);
	IConsoleManager::Get().RegisterConsoleCommand(TEXT("r.ResetRenderTargetsExtent"), TEXT("To reset internal render target extents"), ECVF_Cheat);
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

#if WITH_DUMPGPU
	IConsoleManager::Get().RegisterConsoleCommand(TEXT("DumpGPU"), TEXT("Dump one frame of rendering intermediary resources to disk."), ECVF_Cheat);
#endif

#if WITH_GPUDEBUGCRASH
	IConsoleManager::Get().RegisterConsoleCommand(TEXT("GPUDebugCrash"), TEXT("Crash GPU intentionally for debugging."), ECVF_Cheat);
#endif

#if UE_ENABLE_ARRAY_SLACK_TRACKING
	IConsoleManager::Get().RegisterConsoleCommand(TEXT("SlackReport"),
		TEXT("Generate an array slack memory report to Saved/Logs/SlackReport.  TSV format can be loaded as a spreadsheet.\nUage: SlackReport [Filename] [-Stack=N] [-Verbose=0,1]\nIf no filename, writes to a default filename which increments each report.\nStack setting specifies number of stack frames to consider when grouping allocations"),
		ECVF_Default);
#endif

#if !UE_BUILD_SHIPPING
	IConsoleManager::Get().RegisterConsoleCommand( TEXT( "RedirectToFile" ),
		TEXT( "Creates a file inside Project's Saved folder and outputs command result into it as well as into the log.\n" )
		TEXT( "Usage: RedirectToFile <filepath/filename> <command> [command arguments]\n" )
		TEXT( "Example: RedirectToFile Profiling/CSV/objlist.csv obj list -csv -all\n" )
		TEXT( "Directory structure under Project/Saved folder specified by <filepath> will be created for you if it doesn't exist." ),
		ECVF_Default );

	IConsoleManager::Get().RegisterConsoleCommand(TEXT("DumpCVars"),
		TEXT("List all console variables, their values, and how they were last modified.\n")
		TEXT("Usage: DumpCVars [prefix] [-showhelp] [-deprecated] [-csv[=path]]\n")
		TEXT("prefix      - Only list variables that start with this string\n")
		TEXT("-showhelp   - Show help information for each variable\n")
		TEXT("-deprecated - Only list deprecated variables. Values are not shown for deprecated variables.\n")
		TEXT("-csv        - Save results to a csv file. If a file is not specified, it will be created in the Project Logs directory."),
		ECVF_Default);

	IConsoleManager::Get().RegisterConsoleCommand(TEXT("DumpCCmds"),
		TEXT("List all registered console commands.\n")
		TEXT("Not all commands are registered. Unregistered commands will not be listed by this command\n")
		TEXT("Use DumpConsoleCommands if you need to list unregistered commands as well\n")
		TEXT("Usage: DumpCCmds [prefix] [-showhelp] [-deprecated] [-csv[=path]]\n")
		TEXT("prefix      - Only list commands that start with this string\n")
		TEXT("-showhelp   - Show help information for each command\n")
		TEXT("-deprecated - Only list deprecated commands\n")
		TEXT("-csv        - Save results to a csv file. If a file is not specified, it will be created in the Project Logs directory."),
		ECVF_Default);

	IConsoleManager::Get().RegisterConsoleCommand(TEXT("DumpConsoleCommands"),
		TEXT("List all console commands, both registered and unregistered\n")
		TEXT("Usage: DumpConsoleCommands [prefix]\n")
		TEXT("prefix      - Only list commands that start with this string\n"),
		ECVF_Default);
#endif // !UE_BUILD_SHIPPING

#endif
}

// Naming conventions:
//
// Console variable should start with (suggestion):
//
// r.      Renderer / 3D Engine / graphical feature
// RHI.    Low level RHI (rendering platform) specific
// a.	   Animation
// s. 	   Sound / Music
// n.      Network
// ai.     Artificial intelligence
// i.      Input e.g. mouse/keyboard
// p.      Physics
// t.      Timer
// log.	   Logging system
// con.	   Console (in game  or editor) 
// g.      Game specific
// Compat.
// FX.     Particle effects
// sg.     scalability group (used by scalability system, ini load/save or using SCALABILITY console command)

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
static TAutoConsoleVariable<FString> CVarFreezeAtPosition(
	TEXT("FreezeAtPosition"),
	TEXT(""),	// default value is empty
	TEXT("This console variable stores the position and rotation for the FreezeAt command which allows\n"
		 "to lock the camera in order to provide more deterministic render profiling.\n"
		 "The FreezeAtPosition can be set in the ConsoleVariables.ini (start the map with MAPNAME?bTourist=1).\n"
		 "Also see the FreezeAt command console command.\n"
		 "The number syntax if the same as the one used by the BugIt command:\n"
		 " The first three values define the position, the next three define the rotation.\n"
		 "Example:\n"
		 " FreezeAtPosition 2819.5520 416.2633 75.1500 65378 -25879 0"),
	ECVF_Cheat);

static TAutoConsoleVariable<int32> CVarLimitRenderingFeatures(
	TEXT("r.LimitRenderingFeatures"),
	0,
	TEXT("Allows to quickly reduce render feature to increase render performance.\n"
		 "This is just a quick way to alter multiple show flags and console variables in the game\n"
		 "Disabled more feature the higher the number\n"
		 " <=0:off, order is defined in code (can be documented here when we settled on an order)"),
	ECVF_Cheat | ECVF_RenderThreadSafe);

#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

static TAutoConsoleVariable<int32> CVarUniformBufferPooling(
	TEXT("r.UniformBufferPooling"),
	1,
	TEXT("If we pool object in RHICreateUniformBuffer to have less real API calls to create buffers\n"
		 " 0: off (for debugging)\n"
		 " 1: on (optimization)"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarTranslucentSortPolicy(
	TEXT("r.TranslucentSortPolicy"),
	0,
	TEXT("0: Sort based on distance from camera centerpoint to bounding sphere centerpoint. (default, best for 3D games)\n"
		 "1: Sort based on projected distance to camera.\n"
		 "2: Sort based on the projection onto a fixed axis. (best for 2D games)"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarMobileHDR(
	TEXT("r.MobileHDR"),
	1,
	TEXT("0: Mobile renders in LDR gamma space. (suggested for unlit games targeting low-end phones)\n"
		 "1: Mobile renders in HDR linear space. (default)"),
	ECVF_RenderThreadSafe | ECVF_ReadOnly | ECVF_MobileShaderChange);

static TAutoConsoleVariable<int32> CVarMobileShadingPath(
	TEXT("r.Mobile.ShadingPath"),
	1,
	TEXT("0: Forward shading\n"
		 "1: Deferred shading (Mobile HDR is required for Deferred) (default)"),
	ECVF_RenderThreadSafe | ECVF_ReadOnly);

static TAutoConsoleVariable<int32> CVarMobileExtendedGBuffer(
	TEXT("r.Mobile.ExtendedGBuffer"),
	0,
	TEXT("0: 128bit GBuffer (default)\n"
		 "1: Enable the 160-bit GBuffer, which supports ClearCoat bottom normals and improved normal encoding with decal normal blending (requires static lighting to be disabled)."),
	ECVF_RenderThreadSafe | ECVF_ReadOnly);

static TAutoConsoleVariable<int32> CVarMobileAllowDeferredShadingOpenGL(
	TEXT("r.Mobile.AllowDeferredShadingOpenGL"),
	1,
	TEXT("0: Do not Allow Deferred Shading on OpenGL\n"
		 "1: Allow Deferred Shading on OpenGL (default)"),
	ECVF_RenderThreadSafe | ECVF_ReadOnly);

static TAutoConsoleVariable<int32> CVarMobileAllowFramebufferFetch(
	TEXT("r.Mobile.AllowFramebufferFetch"),
	0,
	TEXT("0: Multi-pass rendering without using sub-passes in Vulkan or programmable blending on iOS and GLES. (only for a deferred shading) (default)\n"
		 "1: Allow use of sub-passes and programmable blending"),
	ECVF_RenderThreadSafe | ECVF_ReadOnly);

static TAutoConsoleVariable<int32> CVarMobileEnableStaticAndCSMShadowReceivers(
	TEXT("r.Mobile.EnableStaticAndCSMShadowReceivers"),
	1,
	TEXT("0: Primitives can receive only static shadowing from stationary lights.\n"
		 "1: Primitives can receive both CSM and static shadowing from stationary lights. (default)"),
	ECVF_RenderThreadSafe | ECVF_ReadOnly);

static TAutoConsoleVariable<int32> CVarMobileEnableMovableLightCSMShaderCulling(
	TEXT("r.Mobile.EnableMovableLightCSMShaderCulling"),
	1,
	TEXT("0: All primitives lit by movable directional light render with CSM.\n"
		 "1: Primitives lit by movable directional light render with the CSM shader when determined to be within CSM range. (default)"),
	ECVF_RenderThreadSafe | ECVF_ReadOnly);

static TAutoConsoleVariable<int32> CVarMobileUseCSMShaderBranch(
	TEXT("r.Mobile.UseCSMShaderBranch"),
	0,
	TEXT("0: Use two shader permutations for CSM and non-CSM shading. (default)\n"
		 "1: Use a single shader pemutation with a branch in a shader to apply CSM (only with r.AllowStaticLighting=0)"),
	ECVF_RenderThreadSafe | ECVF_ReadOnly);

static TAutoConsoleVariable<int32> CVarMobileAllowDistanceFieldShadows(
	TEXT("r.Mobile.AllowDistanceFieldShadows"),
	1,
	TEXT("0: Do not generate shader permutations to render distance field shadows from stationary directional lights.\n"
		 "1: Generate shader permutations to render distance field shadows from stationary directional lights. (default)"),
	ECVF_RenderThreadSafe | ECVF_ReadOnly);

static TAutoConsoleVariable<int32> CVarMobileFloatPrecisionMode(
	TEXT("r.Mobile.FloatPrecisionMode"),
	0,
	TEXT("0: Use Half-precision ALU & some uniform buffers (default)\n"
		 "1: Half precision ALU, except Full precision for material expressions\n"
		 "2: Force use of high precision ALU & uniform buffers in pixel shaders.\n"),
	ECVF_ReadOnly);

static TAutoConsoleVariable<bool> CVarMobileAllowDitheredLODTransition(
	TEXT("r.Mobile.AllowDitheredLODTransition"),
	0,
	TEXT("Whether to support 'Dithered LOD Transition' material option on mobile platforms"),
	ECVF_ReadOnly | ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarMobileAllowPixelDepthOffset(
	TEXT("r.Mobile.AllowPixelDepthOffset"),
	1,
	TEXT("Whether to allow 'Pixel Depth Offset' in materials for Mobile feature level. Depth modification in pixel shaders may reduce GPU performance"),
	ECVF_ReadOnly | ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarMobileAllowPerPixelShadingModels(
	TEXT("r.Mobile.AllowPerPixelShadingModels"),
	1,
	TEXT("Whether to allow 'Per-Pixel Shader Models (From Material Expression)' in materials for Mobile feature level."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarMobileEnabledShadingModelsMask(
	TEXT("r.Mobile.ShadingModelsMask"),
	0xFFFFFFFF,
	TEXT("The mask that indicates which shading models are enabled on mobile platforms."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarMobileForwardEnableLocalLights(
	TEXT("r.Mobile.Forward.EnableLocalLights"),
	1,
	TEXT("0: Local Lights Disabled (default)\n"
		"1: Local Lights Enabled\n"),
	ECVF_ReadOnly | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarMobileRenderRectLightsAsSpotLights(
	TEXT("r.Mobile.Forward.RenderRectLightsAsSpotLights"),
	1,
	TEXT("0: Rect Lights are rendered as area lights\n"
		"1: Rect Lights are rendered as Spot Lights (default)\n"),
	ECVF_ReadOnly | ECVF_MobileShaderChange);

static TAutoConsoleVariable<int32> CVarMobileForwardEnableIESProfiles(
	TEXT("r.Mobile.Forward.EnableIESProfiles"),
	0,
	TEXT("0: IES profiles disabled in mobile forward (default)\n"
		"1: IES profiles enabled in mobile forward \n"),
	ECVF_ReadOnly | ECVF_MobileShaderChange);

static TAutoConsoleVariable<int32> CVarMobileForwardEnableParticleLights(
	TEXT("r.Mobile.Forward.EnableParticleLights"),
	0,
	TEXT("0: Particle Lights Disabled (default)\n"
		 "1: Particle Lights Enabled (requires local lights)\n"),
	ECVF_ReadOnly | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarMobileForwardEnableClusteredReflections(
	TEXT("r.Mobile.Forward.EnableClusteredReflections"),
	0,
	TEXT("Whether to enable clustered reflections on mobile forward, it's always supported on mobile deferred."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarMobileSupportGPUScene(
	TEXT("r.Mobile.SupportGPUScene"),
	1,
	TEXT("Whether to support GPU scene, required for auto-instancing (only Mobile feature level)"),
	ECVF_ReadOnly | ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarSetClearSceneMethod(
	TEXT("r.ClearSceneMethod"),
	1,
	TEXT("Select how the g-buffer is cleared in game mode (only affects deferred shading).\n"
		 " 0: No clear\n"
		 " 1: RHIClear (default)\n"
		 " 2: Quad at max z"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarLocalExposure(
	TEXT("r.LocalExposure"),
	1,
	TEXT("Whether to support local exposure"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarBloomQuality(
	TEXT("r.BloomQuality"),
	5,
	TEXT(" 0: off, no performance impact.\n"
		 " 1: average quality, least performance impact.\n"
		 " 2: average quality, least performance impact.\n"
		 " 3: good quality.\n"
		 " 4: good quality.\n"
		 " 5: Best quality, most significant performance impact. (default)\n"
		 ">5: force experimental higher quality on mobile (can be quite slow on some hardware)"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarSceneColorFringeQuality(
	TEXT("r.SceneColorFringeQuality"),
	1,
	TEXT(" 0: off but best for performance\n"
		 " 1: 3 texture samples (default)"),
	ECVF_Scalability | ECVF_RenderThreadSafe);


// ---------------------------------------

static TAutoConsoleVariable<float> CVarAmbientOcclusionRadiusScale(
	TEXT("r.AmbientOcclusionRadiusScale"),
	1.0f,
	TEXT("Allows to scale the ambient occlusion radius (SSAO).\n"
		 " 0:off, 1.0:normal, <1:smaller, >1:larger"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarAmbientOcclusionStaticFraction(TEXT("r.AmbientOcclusionStaticFraction"),
	-1.0f,
	TEXT("Allows to override the Ambient Occlusion Static Fraction (see post process volume). Fractions are between 0 and 1.\n"
		 "<0: use default setting (default -1)\n"
		 " 0: no effect on static lighting, 0 is free meaning no extra rendering pass\n"
		 " 1: AO affects the stat lighting"),
	ECVF_Default);

static TAutoConsoleVariable<int32> CVarShadowQuality(
	TEXT("r.ShadowQuality"),
	5,
	TEXT("Defines the shadow method which allows to adjust for quality or performance.\n"
		 " 0:off, 1:low(unfiltered), 2:low .. 5:max (default)"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarMotionBlurQuality(
	TEXT("r.MotionBlurQuality"),
	4,
	TEXT("Defines the motion blur method which allows to adjust for quality or performance.\n"
		 " 0:off, 1:low, 2:medium, 3:high (default), 4: very high"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarFullscreenMode(
	TEXT("r.FullScreenMode"),
	1,
	TEXT("Defines how we do full screen when requested (e.g. command line option -fullscreen or in ini [SystemSettings] fullscreen=true)\n"
		 " 0: normal full screen (renders faster, more control over vsync, less GPU memory, 10bit color if possible)\n"
		 " 1: windowed full screen (quick switch between applications and window mode, slight performance loss)\n"
		 " any other number behaves like 0"),
	ECVF_Scalability);

static TAutoConsoleVariable<int32> CVarSceneColorFormat(
	TEXT("r.SceneColorFormat"),
	4,
	TEXT("Defines the memory layout (RGBA) used for the scene color\n"
		 "(affects performance, mostly through bandwidth, quality especially with translucency).\n"
		 " 0: PF_B8G8R8A8 32Bit (mostly for testing, likely to unusable with HDR)\n"
		 " 1: PF_A2B10G10R10 32Bit\n"
		 " 2: PF_FloatR11G11B10 32Bit\n"
		 " 3: PF_FloatRGB 32Bit\n"
		 " 4: PF_FloatRGBA 64Bit (default, might be overkill, especially if translucency is mostly using SeparateTranslucency)\n"
		 " 5: PF_A32B32G32R32F 128Bit (unreasonable but good for testing)"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarMobileSceneColorFormat(
	TEXT("r.Mobile.SceneColorFormat"),
	0,
	TEXT("Overrides the memory layout (RGBA) used for the scene color of the mobile renderer.\nUnsupported overridden formats silently use default"
		 " 0: (default) Automatically select the appropriate format depending on project settings and device support.\n"
		 " 1: PF_FloatRGBA 64Bit \n"
		 " 2: PF_FloatR11G11B10 32Bit\n"
		 " 3: PF_B8G8R8A8 32Bit"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarPostProcessingColorFormat(
	TEXT("r.PostProcessingColorFormat"),
	0,
	TEXT("Defines the memory layout (RGBA) used for most of the post processing chain buffers.\n"
		 " 0: Default\n"
		 " 1: Force PF_A32B32G32R32F 128Bit (unreasonable but good for testing)"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarDepthOfFieldQuality(
	TEXT("r.DepthOfFieldQuality"),
	2,
	TEXT("Allows to adjust the depth of field quality. Currently only fully affects BokehDOF. GaussianDOF is either 0 for off, otherwise on.\n"
		 " 0: Off\n"
		 " 1: Low\n"
		 " 2: high quality (default, adaptive, can be 4x slower)\n"
		 " 3: very high quality, intended for non realtime cutscenes, CircleDOF only (slow)\n"
		 " 4: extremely high quality, intended for non realtime cutscenes, CircleDOF only (very slow)"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarHighResScreenshotDelay(
	TEXT("r.HighResScreenshotDelay"),
	4,
	TEXT("When high-res screenshots are requested there is a small delay to allow temporal effects to converge.\n"
		 "Default: 4. Using a value below the default will disable TemporalAA for improved image quality."),
	ECVF_Default);

static TAutoConsoleVariable<int32> CVarMaterialQualityLevel(
	TEXT("r.MaterialQualityLevel"),
	1,
	TEXT("0 corresponds to low quality materials, as defined by quality switches in materials, 1 corresponds to high, 2 for medium, and 3 for Epic."),
	ECVF_Scalability | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarMaterialShadingPathNodeOverride(
	TEXT("r.Material.ShadingPathNodeOverride"),
	-1,
	TEXT("Material Shading Path Node Override. -1 = no override (default value), 0 = deferred rendering, 1 = forward, 2 = mobile)"),
	ECVF_ReadOnly | ECVF_Preview | ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<bool> CVarMaterialSimulateMobileForLowQuality(
	TEXT("r.Material.SimulateMobileForLowQuality"),
	false,
	TEXT("If true, use the ES3_1 pin on Feature Level Switch nodes and Mobile pin on Shading Path nodes for higher feature levels using low material quality"),
	ECVF_ReadOnly
);

static TAutoConsoleVariable<int32> CVarUseDXT5NormalMaps(
	TEXT("Compat.UseDXT5NormalMaps"),
	0,
	TEXT("Whether to use DXT5 for normal maps, otherwise BC5 will be used, which is not supported on all hardware.\n"
		 "Both formats require the same amount of memory (if driver doesn't emulate the format).\n"
		 "Changing this will cause normal maps to be recompressed on next load (or when using recompile shaders)\n"
		 " 0: Use BC5 texture format (default)\n"
		 " 1: Use DXT5 texture format (lower quality)"),
	// 
	// Changing this causes a full shader recompile
	ECVF_ReadOnly);

static TAutoConsoleVariable<int32> CVarContactShadows(
	TEXT("r.ContactShadows"),
	1,
	TEXT(" 0: disabled.\n"
		 " 1: enabled.\n"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarContactShadowsNonShadowCastingIntensity(
	TEXT("r.ContactShadows.NonShadowCastingIntensity"),
	0.0f,
	TEXT("DEPRECATED. Please use the parameters on the Light Component directly instead.\n"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

// Changing this causes a full shader recompile
static TAutoConsoleVariable<int32> CVarAllowStaticLighting(
	TEXT("r.AllowStaticLighting"),
	1,
	TEXT("Whether to allow any static lighting to be generated and used, like lightmaps and shadowmaps.\n"
		 "Games that only use dynamic lighting should set this to 0 to save some static lighting overhead."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarAllowStaticLightingInWorldPartitionMaps(
	TEXT("r.AllowStaticLightingInWorldPartitionMaps"),
	0,
	TEXT("Whether to allow any static lighting to be generated and used in WorldPartition maps, like lightmaps and shadowmaps.\n"
		 "Games that only use dynamic lighting should set this to 0 to save some static lighting overhead."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe);



static TAutoConsoleVariable<int32> CVarNormalMaps(
	TEXT("r.NormalMapsForStaticLighting"),
	0,
	TEXT("Whether to allow any static lighting to use normal maps for lighting computations."),
	ECVF_Default);

static TAutoConsoleVariable<int32> CVarNumBufferedOcclusionQueries(
	TEXT("r.NumBufferedOcclusionQueries"),
	1,
	TEXT("Number of frames to buffer occlusion queries (including the current renderthread frame).\n"
		 "More frames reduces the chance of stalling the CPU waiting for results, but increases out of date query artifacts."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarMinLogVerbosity(
	TEXT("con.MinLogVerbosity"),
	0,
	TEXT("Allows to see the log in the in game console (by default deactivated to avoid spam and minor performance loss).\n"
		 " 0: no logging other than console response (default)\n"
		 " 1: Only fatal errors (no that useful)\n"
		 " 2: additionally errors\n"
		 " 3: additionally warnings\n"
		 " 4: additionally display\n"
		 " 5: additionally log\n"
		 "..\n"
		 ">=7: all"),
	ECVF_Default);

static TAutoConsoleVariable<int32> CVarMSAACompositingSampleCount(
	TEXT("r.MSAA.CompositingSampleCount"),
	4,
	TEXT("Affects the render quality of the editor 3d objects.\n"
		 " 1: no MSAA, lowest quality\n"
		 " 2: 2x MSAA, medium quality (medium GPU memory consumption)\n"
		 " 4: 4x MSAA, high quality (high GPU memory consumption)\n"
		 " 8: 8x MSAA, very high quality (insane GPU memory consumption)"),
	ECVF_Scalability | ECVF_RenderThreadSafe);


static TAutoConsoleVariable<float> CVarNetPackageMapLongLoadThreshhold(
	TEXT("net.PackageMap.LongLoadThreshhold"),
	0.02f,
	TEXT("Threshhold time in seconds for printing long load warnings in object serialization"),
	ECVF_Default);

static TAutoConsoleVariable<int32> CVarNetPackageMapDebugAllObjects(
	TEXT("net.PackageMap.DebugAll"),
	0,
	TEXT("Debugs PackageMap serialization of all objects"),	
	ECVF_Default);

static TAutoConsoleVariable<FString> CVarNetPackageMapDebugObject(
	TEXT("net.PackageMap.DebugObject"),
	TEXT(""),
	TEXT("Debugs PackageMap serialization of object"
		 "Partial name of object to debug"),
	ECVF_Default);

static TAutoConsoleVariable<int32> CVarNetMontageDebug(
	TEXT("net.Montage.Debug"),
	0,
	TEXT("Prints Replication information about AnimMontages\n"
		 " 0: no print.\n"
		 " 1: Print AnimMontage info on client side as they are played."),
	ECVF_Cheat);

static TAutoConsoleVariable<int32> CVarRenderTargetPoolMin(
	TEXT("r.RenderTargetPoolMin"),
	400,
	TEXT("If the render target pool size (in MB) is below this number there is no deallocation of rendertargets"
		 "Default is 200 MB."),
	ECVF_Scalability | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarIdleWhenNotForeground(
	TEXT("t.IdleWhenNotForeground"), 0,
	TEXT("Prevents the engine from taking any CPU or GPU time while not the foreground app."),
	ECVF_Cheat);

static TAutoConsoleVariable<int32> CVarSetVSyncEnabled(
	TEXT("r.VSync"),
	0,
	TEXT("0: VSync is disabled.(default)\n"
		 "1: VSync is enabled."),
	ECVF_Scalability | ECVF_RenderThreadSafe);

#if WITH_EDITOR
static TAutoConsoleVariable<int32> CVarSetVSyncEditorEnabled(
	TEXT("r.VSyncEditor"),
	0,
	TEXT("0: VSync is disabled in editor.(default)\n"
		 "1: VSync is enabled in editor."),
	ECVF_RenderThreadSafe);
#endif

static TAutoConsoleVariable<int32> CVarSkipPresentOnCameraCut(
	TEXT("r.SkipPresentOnCameraCut"),
	0,
	TEXT("0: Do not skip the Present call on a Camera Cut. (default)\n"
		"1: Skip the Present call on a Camera Cut."),
	ECVF_Scalability | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarFinishCurrentFrame(
	TEXT("r.FinishCurrentFrame"),
	0,
	TEXT(" 0: Wait for previous frame to be Finished on the GPU.(default)\n"
		"-1: Do no wait for GPU.\n"
		" 1: Wait for current frame to be Finished on the GPU."),
	ECVF_Scalability | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarMaxAnistropy(
	TEXT("r.MaxAnisotropy"),
	4,
	TEXT("MaxAnisotropy should range from 1 to 16. Higher values mean better texure quality when using anisotropic filtering but at a cost to performance. Default is 4."),
	ECVF_Scalability | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarShadowMaxResolution(
	TEXT("r.Shadow.MaxResolution"),
	2048,
	TEXT("Max square dimensions (in texels) allowed for rendering shadow depths. Range 4 to hardware limit. Higher = better quality shadows but at a performance cost."),
	ECVF_Scalability | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarShadowMaxCSMShadowResolution(
	TEXT("r.Shadow.MaxCSMResolution"),
	2048,
	TEXT("Max square dimensions (in texels) allowed for rendering Cascaded Shadow depths. Range 4 to hardware limit. Higher = better quality shadows but at a performance cost."),
	ECVF_Scalability | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarShadowCSMTransitionScale(
	TEXT("r.Shadow.CSM.TransitionScale"),
	1.0f,
	TEXT("Allows to scale the cascaded shadow map transition region. Clamped within 0..2.\n"
		 "0: no transition (fastest)\n"
		 "1: as specific in the light settings (default)\n"
		 "2: 2x larger than what was specified in the light"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarMobileContentScaleFactor(
	TEXT("r.MobileContentScaleFactor"),
	1.0f,
	TEXT("Content scale multiplier (equates to iOS's contentScaleFactor to support Retina displays"),
	ECVF_Default);

static TAutoConsoleVariable<int32> CVarMobileDesiredResX(
	TEXT("r.Mobile.DesiredResX"),
	0,
	TEXT("Desired mobile X resolution (longest axis) (non-zero == use for X, calculate Y to retain aspect ratio)"),
	ECVF_Default);

static TAutoConsoleVariable<int32> CVarMobileDesiredResY(
	TEXT("r.Mobile.DesiredResY"),
	0,
	TEXT("Desired mobile Y resolution (shortest axis) (non-zero == use for Y, calculate X to retain aspect ratio)"),
	ECVF_Default);

static TAutoConsoleVariable<int32> CVarLWCTruncateMode(
	TEXT("r.MaterialEditor.LWCTruncateMode"),
	2,
	TEXT("Whether or not the material compiler respects the truncate LWC node or automatic transforms.\n"
		"0: no truncate (LWC always used even if asked to truncate)\n"
		"1: respect the truncate LWC node\n"
		"2: respect the truncate LWC node and automatic transforms"),
	ECVF_ReadOnly);

// this cvar can be removed in shipping to not compile shaders for development (faster)
static TAutoConsoleVariable<int32> CVarCompileShadersForDevelopment(
	TEXT("r.CompileShadersForDevelopment"),
	1,
	TEXT("Setting this to 0 allows to ship a game with more optimized shaders as some\n"
		 "editor and development features are not longer compiled into the shaders.\n"
		 " Note: This should be done when shipping but it's not done automatically yet (feature need to mature\n"
		 "       and shaders will compile slower as shader caching from development isn't shared).\n"
		 "Cannot be changed at runtime - can be put into BaseEngine.ini\n"
		 " 0: off, shader can run a bit faster\n"
		 " 1: on (Default)"),
	ECVF_ReadOnly);

static TAutoConsoleVariable<int32> CVarDontLimitOnBattery(
	TEXT("r.DontLimitOnBattery"),
	0,
	TEXT("0: Limit performance on devices with a battery.(default)\n"
		 "1: Do not limit performance due to device having a battery."),
	ECVF_Scalability | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarViewDistanceScale(
	TEXT("r.ViewDistanceScale"),
	1.0f,
	TEXT("Controls the view distance scale. A primitive's MaxDrawDistance is scaled by this value.\n"
		 "Higher values will increase view distance but at a performance cost.\n"
		 "Default = 1."),
	ECVF_Scalability | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarViewDistanceScaleApplySecondaryScale(
	TEXT("r.ViewDistanceScale.ApplySecondaryScale"),
	0,
	TEXT("If true applies the secondary view distance scale to primitive draw distances.\n"
		 "Default = 0."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarViewDistanceScaleSecondaryScale(
	TEXT("r.ViewDistanceScale.SecondaryScale"),
	1.0f,
	TEXT("Controls the secondary view distance scale, Default = 1.0.\n"
		 "This is an optional scale intended to allow some features or gamemodes to opt-in.\n"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarViewDistanceScale_FieldOfViewMinAngle(
	TEXT("r.ViewDistanceScale.FieldOfViewMinAngle"),
	45.0f,
	TEXT("Scales the scene view distance scale with camera field of view.\n"
		 "Minimum angle of the blend range.\n"
		 "Applies the minimum scale when the camera is at or below this angle."),
	ECVF_Scalability | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarViewDistanceScale_FieldOfViewMinAngleScale(
	TEXT("r.ViewDistanceScale.FieldOfViewMinAngleScale"),
	1.0f,
	TEXT("Scales the scene view distance scale with camera field of view.\n"
		 "This value is applied when the camera is at or below the minimum angle."),
	ECVF_Scalability | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarViewDistanceScale_FieldOfViewMaxAngle(
	TEXT("r.ViewDistanceScale.FieldOfViewMaxAngle"),
	90.0f,
	TEXT("Scales the scene view distance scale with camera field of view.\n"
		 "Maximum angle of the blend range.\n"
		 "Applies the maximum scale when the camera is at or above this angle."),
	ECVF_Scalability | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarViewDistanceScale_FieldOfViewMaxAngleScale(
	TEXT("r.ViewDistanceScale.FieldOfViewMaxAngleScale"),
	1.0f,
	TEXT("Scales the scene view distance scale with camera field of view.\n"
		 "This value is applied when the camera is at or above the maximum angle."),
	ECVF_Scalability | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarViewDistanceScale_FieldOfViewAffectsHLOD(
	TEXT("r.ViewDistanceScale.FieldOfViewAffectsHLOD"),
	0,
	TEXT("If enabled, applies the field of view scaling to HLOD draw distances as well as non-HLODs."),
	ECVF_Scalability | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarViewDistanceScale_SkeletalMeshOverlay(
	TEXT("r.ViewDistanceScale.SkeletalMeshOverlay"),
	1.f,
	TEXT("Controls the distance scale for skeletal mesh overlay, Default = 1.0. \n"
		 "Higher values will increase skeletal mesh overlay draw distance. This value is applied together with r.ViewDistanceScale"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarLightFunctionQuality(
	TEXT("r.LightFunctionQuality"),
	2,
	TEXT("Defines the light function quality which allows to adjust for quality or performance.\n"
		 "<=0: off (fastest)\n"
		 "  1: low quality (e.g. half res with blurring, not yet implemented)\n"
		 "  2: normal quality (default)\n"
		 "  3: high quality (e.g. super-sampled or colored, not yet implemented)"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarEyeAdaptationQuality(
	TEXT("r.EyeAdaptationQuality"),
	2,
	TEXT("Defines the eye adaptation quality which allows to adjust for quality or performance.\n"
		 "<=0: off (fastest)\n"
		 "  1: low quality (e.g. non histogram based, not yet implemented)\n"
		 "  2: normal quality (default)\n"
		 "  3: high quality (e.g. screen position localized, not yet implemented)"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarShadowDistanceScale(
	TEXT("r.Shadow.DistanceScale"),
	1.0f,
	TEXT("Scalability option to trade shadow distance versus performance for directional lights (clamped within a reasonable range).\n"
		 "<1: shorter distance\n"
		 " 1: normal (default)\n"
		 ">1: larger distance"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarFreeSkeletalMeshBuffers(
	TEXT("r.FreeSkeletalMeshBuffers"),
	0,
	TEXT("Controls whether skeletal mesh buffers are kept in CPU memory to support merging of skeletal meshes.\n"
		 "0: Keep buffers(default)\n"
		 "1: Free buffers"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarDetailMode(
	TEXT("r.DetailMode"),
	3,
	TEXT("Current detail mode; determines whether components of actors should be updated/ ticked.\n"
		" 0: low, show objects with DetailMode low\n"
		" 1: medium, show objects with DetailMode medium or below\n"
		" 2: high, show objects with DetailMode high or below\n"
		" 3: epic, show all objects (default)"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarCookOutUnusedDetailModeComponents(
	TEXT("r.CookOutUnusedDetailModeComponents"),
	0,
	TEXT("If set, components which are not relevant for the current detail mode will be cooked out.\n"
		 " 0: keep components even if not relevant for the current detail mode.\n"
		 " 1: cook out components not relevant for the current detail mode.\n"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarDBuffer(
	TEXT("r.DBuffer"),
	1,
	TEXT("Enables DBuffer decal material blend modes.\n"
		 "DBuffer decals are rendered before the base pass, allowing them to affect static lighting and skylighting correctly. \n"
		 "When enabled, a full prepass will be forced which adds CPU / GPU cost.  Several texture lookups will be done in the base pass to fetch the decal properties, which adds pixel work.\n"
		 " 0: off\n"
		 " 1: on (default)"),
	ECVF_RenderThreadSafe | ECVF_ReadOnly);

static TAutoConsoleVariable<int32> CVarMobileDBuffer(
	TEXT("r.Mobile.DBuffer"),
	0,
	TEXT("Enables DBuffer decal material blend modes when using the mobile forward renderer.\n"
		"DBuffer decals are rendered before the base pass, allowing them to affect static lighting and skylighting correctly. \n"
		"When enabled, a full prepass will be forced which adds CPU / GPU cost.  Several texture lookups will be done in the base pass to fetch the decal properties, which adds pixel work.\n"
		" 0: off (default)\n"
		" 1: on"),
	ECVF_RenderThreadSafe | ECVF_ReadOnly);

static TAutoConsoleVariable<int32> CVarMobileForwardDecalLighting(
	TEXT("r.Mobile.Forward.DecalLighting"),
	1,
	TEXT("Enables lit decals when using the mobile forward renderer without DBuffer.\n"
		" 0: off\n"
		" 1: on, lighting channel 1 is used for decal lighting\n"
		" 2: on, lighting channel 2 is used for decal lighting\n"
		" 3: on, lighting channel 3 is used for decal lighting"),
	ECVF_RenderThreadSafe | ECVF_ReadOnly);

static TAutoConsoleVariable<float> CVarSkeletalMeshLODRadiusScale(
	TEXT("r.SkeletalMeshLODRadiusScale"),
	1.0f,
	TEXT("Scale factor for the screen radius used in computing discrete LOD for skeletal meshes. (0.25-1)"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarPreTileTextures(
	TEXT("r.PreTileTextures"),
	1,
	TEXT("If set to 1, textures will be tiled during cook and are expected to be cooked at runtime"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarPrecomputedVisibilityWarning(
	TEXT("r.PrecomputedVisibilityWarning"),
	0,
	TEXT("If set to 1, a warning will be displayed when rendering a scene from a view point without precomputed visibility."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarDemotedLocalMemoryWarning(
	TEXT("r.DemotedLocalMemoryWarning"),
	1,
	TEXT("If set to 1, a warning will be displayed when local memory has been demoted to system memory."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarFeatureLevelPreview(
	TEXT("r.FeatureLevelPreview"),
	0,
	TEXT("If 1 the quick settings menu will contain an option to enable feature level preview modes"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarVerifyPeer(
	TEXT("n.VerifyPeer"),
	1,
	TEXT("Sets libcurl's CURLOPT_SSL_VERIFYPEER option to verify authenticity of the peer's certificate.\n"
		 "  0 = disable (allows self-signed certificates)\n"
		 "  1 = enable [default]"),
	ECVF_ReadOnly);

static TAutoConsoleVariable<float> CVarEmitterSpawnRateScale(
	TEXT("r.EmitterSpawnRateScale"),
	1.0,
	TEXT("A global scale upon the spawn rate of emitters. Emitters can choose to apply or ignore it via their bApplyGlobalSpawnRateScale property."),
	ECVF_Scalability | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarCheckSRVTransitions(
	TEXT("r.CheckSRVTransitions"),
	0,
	TEXT("Tests that render targets are properly transitioned to SRV when SRVs are set."),
	ECVF_RenderThreadSafe);  

static TAutoConsoleVariable<int32> CVarDisableThreadedRendering(
	TEXT("r.AndroidDisableThreadedRendering"),
	0,
	TEXT("Sets whether or not to allow threaded rendering for a particular Android device profile.\n"
		 "	0 = Allow threaded rendering [default]\n"
		 "	1 = Disable creation of render thread on startup"),
	ECVF_ReadOnly);


static TAutoConsoleVariable<int32> CVarDisableThreadedRenderingFirstLoad(
	TEXT("r.AndroidDisableThreadedRenderingFirstLoad"),
	0,
	TEXT("Sets whether or not to allow threaded rendering for a particular Android device profile on the initial load.\n"
		 "	0 = Allow threaded rendering on the initial load [default]\n"
		 "	1 = Disable threaded rendering on the initial load"),
	ECVF_ReadOnly);

static TAutoConsoleVariable<int32> CVarDisableVulkanSupport(
	TEXT("r.Android.DisableVulkanSupport"),
	0,
	TEXT("Disable support for vulkan API. (Android Only)\n"
		 "  0 = vulkan API will be used (providing device and project supports it) [default]\n"
		 "  1 = vulkan will be disabled, opengl fall back will be used."),
	ECVF_ReadOnly);

static TAutoConsoleVariable<int32> CVarDisableVulkanSM5Support(
	TEXT("r.Android.DisableVulkanSM5Support"),
	0,
	TEXT("Disable support for vulkan API. (Android Only)\n"
		 "  0 = Vulkan SM5 API will be used (providing device and project supports it) [default]\n"
		 "  1 = Vulkan SM5 will be disabled, Vulkan or OpenGL fall back will be used."),
	ECVF_ReadOnly);

static TAutoConsoleVariable<int32> CVarDisableOpenGLES31Support(
	TEXT("r.Android.DisableOpenGLES31Support"),
	0,
	TEXT("Disable support for OpenGLES 3.1 API. (Android Only)\n"
		 "  0 = OpenGLES 3.1 API will be used (providing device and project supports it) [default]\n"
		 "  1 = OpenGLES 3.1 will be disabled, Vulkan will be used."),
	ECVF_ReadOnly);

static TAutoConsoleVariable<int32> CVarDisableAndroidGLASTCSupport(
	TEXT("r.Android.DisableASTCSupport"),
	0,
	TEXT("Disable support for ASTC Texture compression if OpenGL driver supports it. (Android Only)\n"
		 "  0 = ASTC texture compression will be used if driver supports it [default]\n"
		 "  1 = ASTC texture compression will not be used."),
	ECVF_ReadOnly);

static TAutoConsoleVariable<int32> CVarDisableOpenGLTextureStreamingSupport(
	TEXT("r.OpenGL.DisableTextureStreamingSupport"),
	0,
	TEXT("Disable support for texture streaming on OpenGL.\n"
		 "  0 = Texture streaming will be used if device supports it [default]\n"
		 "  1 = Texture streaming will be disabled."),
	ECVF_ReadOnly);

static TAutoConsoleVariable<int32> CVarAndroidOverrideExternalTextureSupport(
	TEXT("r.Android.OverrideExternalTextureSupport"),
	0,
	TEXT("Override external texture support for OpenGLES API. (Android Only)\n"
		 "  0 = normal detection used [default]\n"
		 "  1 = disable external texture support\n"
		 "  2 = force ImageExternal100 (version #100 with GL_OES_EGL_image_external)\n"
		 "  3 = force ImageExternal300 (version #300 with GL_OES_EGL_image_external)\n"
		 "  4 = force ImageExternalESSL300 (version #300 with GL_OES_EGL_image_external_essl3)"),
	ECVF_ReadOnly);

static TAutoConsoleVariable<FString> CVarCustomUnsafeZones(
	TEXT("r.CustomUnsafeZones"),
	TEXT(""),
	TEXT("Allows you to set custom unsafe zones. Define them based on Portrait (P) or Landscape (L) for a device oriented 'upright'."
		 "Unsafe zones may be either fixed or free, depending on if they move along with the rotation of the device."
		 "Format is (P:fixed[x1, y1][width, height]), semicolon-separated for each custom unsafe zone. +Values add from 0, -Values subtract from Height or Width"),
	ECVF_Default);

static TAutoConsoleVariable<int32> CVarSkyLightingQuality(
	TEXT("r.SkyLightingQuality"),
	1,
	TEXT("Defines the sky lighting quality which allows to adjust for performance.\n"
		 "<=0: off (fastest)\n"
		 "  1: on\n"),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarMobileDefaultAntiAliasing(
	TEXT("r.Mobile.AntiAliasing"),
	2,
	TEXT("Mobile default AntiAliasingMethod\n"
		 " 0: off (no anti-aliasing)\n"
		 " 1: FXAA (faster than TemporalAA but much more shimmering for non static cases)\n"
		 " 2: TemporalAA (default, it will fallback to FXAA if SupportsGen4TAA is disabled) \n"
		 " 3: MSAA \n"
		 " 4: TSR(it will fallback to TAA if SupportsGen5TAA is disabled) \n"
		 " 5: SMAA"),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<int32> CVarMobileSupportsGen4TAA(
	TEXT("r.Mobile.SupportsGen4TAA"),
	1,
	TEXT("Support desktop Gen4 TAA with mobile rendering\n"
		 "0: Fallback to FXAA"
		 "1: Support Desktop Gen4 TAA (default)"),
	ECVF_ReadOnly | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarLegacyLuminanceFactors(
	TEXT("r.LegacyLuminanceFactors"),
	0,
	TEXT("Default luminance factors:\n"
		"0: Working color space coefficients (default)\n"
		"1: Legacy coefficients (inaccurate). Activate to retain the 5.4 and earlier versions behavior.\n"),
	ECVF_ReadOnly | ECVF_RenderThreadSafe);

class FConsoleObjectShadowData
{
protected:
	
	FConsoleObjectShadowData(
		const TCHAR* CVarToShadow,
		EShadowCVarBehavior InLookupBehavior,
		EShadowCVarBehavior InUsageBehavior,
		bool bInIsDeprecated,
		const TCHAR* InDeprecatedVersion,
		const TCHAR* InDeprecationMessage = nullptr
	)
		: ShadowName(CVarToShadow)
		, DeprecatedVersion(InDeprecatedVersion)
		, DeprecationMessage(InDeprecationMessage)
		, LookupBehavior(InLookupBehavior)
		, UsageBehavior(InUsageBehavior)
		, bHasLooked(false)
		, bHasMessagedForUsage(false)
		, bHasMessagedEditorForUsage(false)
		, bIsDeprecated(bInIsDeprecated)
	{
	}

	virtual ~FConsoleObjectShadowData()
	{
	}

	virtual IConsoleObject* GetRealObject() const = 0;
	virtual void SetRealObject(IConsoleObject* Object) const = 0;
	virtual FString GetThisName() const = 0;

	void LogOrEditorMessage(const FText& Msg, bool bIsError) const
	{
		if (GIsEditor && !bHasMessagedEditorForUsage)
		{
			bHasMessagedEditorForUsage = true;

			FMessageLog EditorErrors("EditorErrors");
			TSharedRef<FTokenizedMessage> Message = EditorErrors.Message(bIsError ? EMessageSeverity::Error : EMessageSeverity::Warning);
			Message->AddToken(FTextToken::Create(Msg));
			EditorErrors.Notify();
		}

		// always spit to log
		if (bIsError)
		{
			UE_LOGF(LogConsoleManager, Error, "%ls", *Msg.ToString());
		}
		else
		{
			UE_LOGF(LogConsoleManager, Warning, "%ls", *Msg.ToString());
		}
	}

	bool Bind() const
	{
		if (GetRealObject() == nullptr)
		{
			// if we looked but it wasn't found, then just return false
			if (bHasLooked)
			{
				return false;
			}
			bHasLooked = true;

			IConsoleObject* Object = IConsoleManager::Get().FindConsoleObject(*ShadowName, false);
			if (Object == nullptr)
			{
				if (LookupBehavior != EShadowCVarBehavior::NoMessaging)
				{
					FFormatNamedArguments Arguments;
					Arguments.Add(TEXT("ThisName"), FText::FromString(GetThisName()));
					Arguments.Add(TEXT("ShadowName"), FText::FromString(ShadowName));

					FText Message;
					if (DeprecationMessage.IsEmpty())
					{
						if (LookupBehavior == EShadowCVarBehavior::Assert)
						{
							Message = FText::Format(LOCTEXT("FailedShadowCVarLookup_Assert", "Attempted to delay-load real CVar '{ThisName}' for shadowed CVar '{ShadowName}' failed."), Arguments);
						}
						else
						{
							Message = FText::Format(LOCTEXT("FailedShadowCVarLookup", "Attempted to delay-load real CVar '{ThisName}' for shadowed CVar '{ShadowName}' failed. Uses of '{ThisName}' will do nothing."), Arguments);
						}
					}
					else
					{
						Message = FText::FromString(DeprecationMessage);
					}

					switch (LookupBehavior)
					{
					case EShadowCVarBehavior::Warn:
						LogOrEditorMessage(Message, false);
						break;
					case EShadowCVarBehavior::Error:
						LogOrEditorMessage(Message, true);
						break;
					case EShadowCVarBehavior::Ensure:
						ensureMsgf(false, TEXT("%s"), *Message.ToString());
						break;
					case EShadowCVarBehavior::Assert:
						UE_LOGF(LogConsoleManager, Fatal, "%ls", *Message.ToString());
						break;
					}
				}
				return false;
			}

			SetRealObject(Object);
		}

		return true;
	}

	bool BindForUsage() const
	{
		if (!Bind())
		{
			return false;
		}

		if (!bHasMessagedForUsage && !FCVarDisableDeprecationWarningScope::IsDisabled() )
		{
			if (UsageBehavior != EShadowCVarBehavior::NoMessaging)
			{
				FFormatNamedArguments Arguments;
				Arguments.Add(TEXT("ThisName"), FText::FromString(GetThisName()));
				Arguments.Add(TEXT("ShadowName"), FText::FromString(ShadowName));
				Arguments.Add(TEXT("DeprecatedVersion"), FText::FromString(DeprecatedVersion));

				FText Message;
				if (DeprecationMessage.IsEmpty())
				{
					if (ShadowName.IsEmpty() || !!(GetRealObject()->GetFlags() & ECVF_Unregistered))
					{
						// in-place deprecation requires a version number
						Message = FText::Format(LOCTEXT("ShadowCVarUsage_DeprecatedInPlace", "Using a deprecated (as of UE {DeprecatedVersion}) CVar/Command: '{ThisName}'. It will be removed in the future, and may not function as expected. Please review your usage of {ThisName}."), Arguments);
					}
					else
					{
						if (DeprecatedVersion != TEXT(""))
						{
							Message = FText::Format(LOCTEXT("ShadowCVarUsage_Deprecated", "Using a deprecated (as of UE {DeprecatedVersion}) CVar/Command: '{ThisName}'. It will be removed in the future. Change all uses to '{ShadowName}' instead."), Arguments);
						}
						else
						{
							Message = FText::Format(LOCTEXT("ShadowCVarUsage", "Using a shadowed CVar '{ThisName}'. It is recommended to change all uses to '{ShadowName}' instead."), Arguments);
						}
					}
				}
				else
				{
					Message = FText::FromString(DeprecationMessage);
				}

				switch (UsageBehavior)
				{
				case EShadowCVarBehavior::Warn:
					LogOrEditorMessage(Message, false);
					break;
				case EShadowCVarBehavior::Error:
					LogOrEditorMessage(Message, true);
					break;
				case EShadowCVarBehavior::Ensure:
					// don't keep re-ensuring for this cvar
					bHasMessagedForUsage = true;
					ensureAlwaysMsgf(false, TEXT("%s"), *Message.ToString());
					break;
				case EShadowCVarBehavior::Assert:
					UE_LOGF(LogConsoleManager, Fatal, "%ls", *Message.ToString());
					break;
				}
			}
		}

		return true;
	}

	FString ShadowName;
	FString DeprecatedVersion;
	FString DeprecationMessage;
	EShadowCVarBehavior LookupBehavior;
	EShadowCVarBehavior UsageBehavior;
	mutable uint8 bHasLooked : 1;
	mutable uint8 bHasMessagedForUsage : 1;
	mutable uint8 bHasMessagedEditorForUsage : 1;
	uint8 bIsDeprecated : 1;
};


class FConsoleVariableShadow : public FConsoleObjectShadowData, public IConsoleVariable
{
public:
	FConsoleVariableShadow(
		const TCHAR* CVarToShadow,
		EShadowCVarBehavior InLookupBehavior,
		EShadowCVarBehavior InUsageBehavior,
		bool bInIsDeprecated,
		const TCHAR* InDeprecatedVersion,
		const TCHAR* InDeprecationMessage
	)
		: FConsoleObjectShadowData(CVarToShadow, InLookupBehavior, InUsageBehavior, bInIsDeprecated, InDeprecatedVersion, InDeprecationMessage)
		, RealVariable(nullptr)
	{
	}

	virtual class IConsoleVariable* AsVariable() override
	{
		return this;
	}

	virtual bool IsShadowObject() const override
	{
		return true;
	}
	virtual IConsoleObject* GetShadowedObject() const override
	{
		if (Bind())
		{
			return RealVariable;
		}
		return nullptr;
	}
	virtual bool IsDeprecated() const override
	{
		return bIsDeprecated;
	}


	virtual bool IsVariableBool() const override
	{
		if (Bind())
		{
			return RealVariable->IsVariableBool();
		}
		return false;
	}
	virtual bool IsVariableInt() const override
	{
		if (Bind())
		{
			return RealVariable->IsVariableInt();
		}
		return false;
	}
	virtual bool IsVariableFloat() const override
	{
		if (Bind())
		{
			return RealVariable->IsVariableFloat();
		}
		return false;
	}
	virtual bool IsVariableString() const override
	{
		if (Bind())
		{
			return RealVariable->IsVariableString();
		}
		return false;
	}

	virtual class TConsoleVariableData<bool>* AsVariableBool() override
	{
		if (BindForUsage())
		{
			return RealVariable->AsVariableBool();
		}
		return nullptr;
	}
	virtual class TConsoleVariableData<int32>* AsVariableInt() override
	{
		if (BindForUsage())
		{
			return RealVariable->AsVariableInt();
		}
		return nullptr;
	}
	virtual class TConsoleVariableData<float>* AsVariableFloat() override
	{
		if (BindForUsage())
		{
			return RealVariable->AsVariableFloat();
		}
		return nullptr;
	}
	virtual class TConsoleVariableData<FString>* AsVariableString() override
	{
		if (BindForUsage())
		{
			return RealVariable->AsVariableString();
		}
		return nullptr;
	}

#if UE_WITH_CVAR_HELP_TEXT
	virtual const TCHAR* GetHelp() const override
	{
		if (Bind())
		{
			return RealVariable->GetHelp();
		}
		return TEXT("");
	}

	virtual FText GetDetailedHelp() const override
	{
		if (Bind())
		{
			return RealVariable->GetDetailedHelp();
		}
		return INVTEXT("");
	}

	virtual void SetHelp(const TCHAR* Value) override
	{
		if (Bind())
		{
			return RealVariable->SetHelp(Value);
		}
	}
#endif

	virtual EConsoleVariableFlags GetFlags() const override
	{
		if (Bind())
		{
			return RealVariable->GetFlags();
		}
		return (EConsoleVariableFlags)0;

	}

	virtual void SetFlags(const EConsoleVariableFlags Value) override
	{
		if (BindForUsage())
		{
			RealVariable->SetFlags(Value);
		}
	}

	virtual FConsoleVariableBase::FResolvedContext ResolveContext(const FConsoleVariableBase::FSetContext& Context) override
	{
		if (BindForUsage())
		{
			return RealVariable->ResolveContext(Context);
		}
		FResolvedContext Resolved;
		Resolved.Flags = Context.Flags;
		Resolved.Tag = Context.Tag;
		return Resolved;
	}

	virtual void Set(const TCHAR* Value, const FConsoleVariableBase::FResolvedContext& Context) override
	{
		if (BindForUsage())
		{
			RealVariable->Set(Value, Context);
		}
	}

	virtual void Unset(EConsoleVariableFlags SetBy, FName Tag) override
	{
		if (BindForUsage())
		{
			RealVariable->Unset(SetBy, Tag);
		}
	}

	virtual bool GetBool() const override
	{
		if (BindForUsage())
		{
			return RealVariable->GetBool();
		}
		return false;
	}

	virtual int32 GetInt() const override
	{
		if (BindForUsage())
		{
			return RealVariable->GetInt();
		}
		return 0;
	}

	virtual float GetFloat() const override
	{
		if (BindForUsage())
		{
			return RealVariable->GetFloat();
		}
		return 0;
	}

	virtual FString GetString() const override
	{
		if (BindForUsage())
		{
			return RealVariable->GetString();
		}
		return FString();
	}

	virtual void SetOnChangedCallback(const FConsoleVariableDelegate& Callback) override
	{
		if (Bind())
		{
			RealVariable->SetOnChangedCallback(Callback);
		}
	}

	virtual void ClearOnChangedCallback() override
	{
		if (Bind())
		{
			RealVariable->ClearOnChangedCallback();
		}
	}

	virtual FConsoleVariableMulticastDelegate& OnChangedDelegate() override
	{
		if (Bind())
		{
			return RealVariable->OnChangedDelegate();
		}
		static FConsoleVariableMulticastDelegate Dummy;
		return Dummy;
	}

	virtual FString GetDefaultValue() override
	{
		if (BindForUsage())
		{
			return RealVariable->GetDefaultValue();
		}
		return FString();
	}

	virtual void LogHistory(FOutputDevice& Ar) override
	{
		if (Bind())
		{
			RealVariable->LogHistory(Ar);
		}
	}

	/**
	 * Track memory used by history data
	 */
	virtual SIZE_T GetHistorySize() override
	{
		if (Bind())
		{
			return RealVariable->GetHistorySize();
		}
		return 0;
	}

	virtual void Release() override
	{
		// the real var should do it's own Release when it's time
		RealVariable = nullptr;
	}

#if ALLOW_OTHER_PLATFORM_CONFIG

	virtual TSharedPtr<IConsoleVariable> GetPlatformValueVariable(FName PlatformName, const FString& DeviceProfileName = FString())
	{
		if (BindForUsage())
		{
			return RealVariable->GetPlatformValueVariable(PlatformName, DeviceProfileName);
		}
		return nullptr;
	}

	virtual bool HasPlatformValueVariable(FName PlatformName, const FString& DeviceProfileName = FString()) override
	{
		if (Bind())
		{
			return RealVariable->HasPlatformValueVariable(PlatformName, DeviceProfileName);
		}
		return false;
	}

	virtual void ClearPlatformVariables(FName PlatformName = NAME_None)
	{
		if (Bind())
		{
			RealVariable->ClearPlatformVariables(PlatformName);
		}
	}

#endif

protected:
	virtual IConsoleObject* GetRealObject() const override
	{
		return RealVariable;
	}
	virtual void SetRealObject(IConsoleObject* Object) const override
	{
		RealVariable = (IConsoleVariable*)Object;
	}
	virtual FString GetThisName() const override
	{
		return IConsoleManager::Get().FindConsoleObjectName(this);
	}

private:

	mutable IConsoleVariable* RealVariable;
};

class FConsoleCommandShadow : public FConsoleObjectShadowData, public IConsoleCommand
{
public:
	FConsoleCommandShadow(
		const TCHAR* CVarToShadow,
		EShadowCVarBehavior InLookupBehavior,
		EShadowCVarBehavior InUsageBehavior,
		bool bInIsDeprecated,
		const TCHAR* InDeprecatedVersion,
		const TCHAR* InDeprecationMessage
	)
		: FConsoleObjectShadowData(CVarToShadow, InLookupBehavior, InUsageBehavior, bInIsDeprecated, InDeprecatedVersion, InDeprecationMessage)
		, RealCommand(nullptr)
	{
	}

	virtual struct IConsoleCommand* AsCommand() override
	{
		return this;
	}

	virtual bool IsDeprecated() const override
	{
		return bIsDeprecated;
	}

#if UE_WITH_CVAR_HELP_TEXT
	virtual const TCHAR* GetHelp() const override
	{
		if (Bind())
		{
			return RealCommand->GetHelp();
		}
		return TEXT("");
	}

	virtual void SetHelp(const TCHAR* InValue) override
	{
		if (Bind())
		{
			return RealCommand->SetHelp(InValue);
		}
	}

	virtual FText GetDetailedHelp() const
	{
		if (Bind())
		{
			return RealCommand->GetDetailedHelp();
		}
		return INVTEXT("");
	}
#endif

	virtual EConsoleVariableFlags GetFlags() const override
	{
		if (Bind())
		{
			return RealCommand->GetFlags();
		}
		return (EConsoleVariableFlags)0;

	}

	virtual void SetFlags(const EConsoleVariableFlags Value) override
	{
		if (BindForUsage())
		{
			RealCommand->SetFlags(Value);
		}
	}

	virtual bool Execute(const TArray< FString >& Args, UWorld* InWorld, class FOutputDevice& OutputDevice) override
	{
		if (BindForUsage())
		{
			return RealCommand->Execute(Args, InWorld, OutputDevice);
		}
		return false;
	}

	virtual void Release() override
	{
		// the real var should do it's own Release when it's time
		RealCommand = nullptr;
	}

protected:
	virtual IConsoleObject* GetRealObject() const override
	{
		return RealCommand;
	}
	virtual void SetRealObject(IConsoleObject* Object) const override
	{
		RealCommand = (IConsoleCommand*)Object;
	}
	virtual FString GetThisName() const override
	{
		return IConsoleManager::Get().FindConsoleObjectName(this);
	}

private:

	mutable IConsoleCommand* RealCommand;
};



FAutoConsoleVariableShadow::FAutoConsoleVariableShadow(const TCHAR* Name, const TCHAR* CVarToShadow, EShadowCVarBehavior LookupFailureBehavior)
{
#if !NO_CVARS
	GetManager().AddShadowConsoleObject(Name, new FConsoleVariableShadow(CVarToShadow, LookupFailureBehavior, EShadowCVarBehavior::NoMessaging, false, nullptr, TEXT("")));
#endif
}

FAutoConsoleVariableDeprecated::FAutoConsoleVariableDeprecated(const TCHAR* Name, const TCHAR* CVarToShadow, const TCHAR* DeprecatedAtVersion, EShadowCVarBehavior UsageBehavior,  EShadowCVarBehavior LookupFailureBehavior, const TCHAR* DeprecationMessage)
{
#if !NO_CVARS
	GetManager().AddShadowConsoleObject(Name, new FConsoleVariableShadow(CVarToShadow, LookupFailureBehavior, UsageBehavior, true, DeprecatedAtVersion, DeprecationMessage));
#endif
}

FAutoConsoleCommandDeprecated::FAutoConsoleCommandDeprecated(const TCHAR* Name, const TCHAR* CVarToShadow, const TCHAR* DeprecatedAtVersion, EShadowCVarBehavior UsageBehavior, EShadowCVarBehavior LookupFailureBehavior, const TCHAR* DeprecationMessage)
{
#if !NO_CVARS
	GetManager().AddShadowConsoleObject(Name, new FConsoleCommandShadow(CVarToShadow, LookupFailureBehavior, UsageBehavior, true, DeprecatedAtVersion, DeprecationMessage));
#endif
}


// helper class for in-place deprecation, which has a backing cvar unregistered cvar created with a _dep nam, but since 
// it's ECVF_Unregistered, it doesn't show up in auto-complete etc
class FConsoleVariableDeprecatedNoReplace : public FConsoleVariableShadow
{
public:
	FConsoleVariableDeprecatedNoReplace(const TCHAR* CVarToShadow, const TCHAR* DeprecatedAtVersion, EShadowCVarBehavior UsageBehavior, const TCHAR* DeprecationMessage)
	: FConsoleVariableShadow(CVarToShadow, EShadowCVarBehavior::NoMessaging, UsageBehavior, true, DeprecatedAtVersion, DeprecationMessage)
	{
		
	}
	
	virtual EConsoleVariableFlags GetFlags() const override
	{
		if (Bind())
		{
			// we need to remove the ECVF_Unregistered flag when returning the info about the "good" cvar
			return (EConsoleVariableFlags)(GetRealObject()->GetFlags() & ~ECVF_Unregistered);
		}
		return (EConsoleVariableFlags)0;

	}
};

class FConsoleCommandDeprecatedNoReplace : public FConsoleCommandShadow
{
public:
	FConsoleCommandDeprecatedNoReplace(const TCHAR* CVarToShadow, const TCHAR* DeprecatedAtVersion, EShadowCVarBehavior UsageBehavior, const TCHAR* DeprecationMessage)
		: FConsoleCommandShadow(CVarToShadow, EShadowCVarBehavior::NoMessaging, UsageBehavior, true, DeprecatedAtVersion, DeprecationMessage)
	{

	}

	virtual EConsoleVariableFlags GetFlags() const override
	{
		if (Bind())
		{
			// we need to remove the ECVF_Unregistered flag when returning the info about the "good" cvar
			return (EConsoleVariableFlags)(GetRealObject()->GetFlags() & ~ECVF_Unregistered);
		}
		return (EConsoleVariableFlags)0;

	}
};

static FString MakeDeprecatedHelp(const TCHAR* Name, const TCHAR* DeprecatedAtVersion, const TCHAR* DeprecationMessage)
{
	FString Message = DeprecationMessage;
	if (Message.IsEmpty())
	{
		Message = FString::Printf(TEXT("'%s' has been deprecated as of UE %s. It should no longer be used."), Name, DeprecatedAtVersion);
	}

	return Message;
}

static thread_local int32 GDisabledCount = 0;

FCVarDisableDeprecationWarningScope::FCVarDisableDeprecationWarningScope(const bool bInCondition) 
	: bCondition(bInCondition)
{
	if (bCondition)
	{
		++GDisabledCount;
	}
}
FCVarDisableDeprecationWarningScope::~FCVarDisableDeprecationWarningScope() 
{
	if (bCondition)
	{
		--GDisabledCount;
	}
}

bool FCVarDisableDeprecationWarningScope::IsDisabled()
{ 
	return GDisabledCount > 0; 
}

template<typename T, typename CreateVarFuncType>
IConsoleVariable* CreateInPlaceDeprecatedVar(const FCVarDeprecationArgs& DeprecationArgs, const TCHAR* Name, uint32 Flags, CreateVarFuncType CreateVarFunc)
{
#if !NO_CVARS
	// create a backing unregistered cvar with a dummy name, but this name won't show up since Unregistered
	FString BackingName = FString(Name) + "_dep";
	IConsoleVariable* BackingCVar = CreateVarFunc(*MakeDeprecatedHelp(Name, DeprecationArgs.Version, DeprecationArgs.Message), (EConsoleVariableFlags)(Flags| ECVF_Unregistered));
	GetManager().AddConsoleObject(*BackingName, BackingCVar);
	
	// create the shadow cvar that shadows the backing cvar, so we can use all the existing logic
	FConsoleVariableDeprecatedNoReplace* DeprecatedCVar = new FConsoleVariableDeprecatedNoReplace(*BackingName, DeprecationArgs.Version, DeprecationArgs.UsageBehavior, DeprecationArgs.Message);
	// don't let the AddCosoleObject throw an error by calling GetFlags()
	GetManager().AddShadowConsoleObject(Name, DeprecatedCVar);

	return BackingCVar;
#else
	return nullptr;
#endif
}

template<typename T>
IConsoleVariable* CreateInPlaceDeprecatedVar(const FCVarDeprecationArgs& DeprecationArgs, const TCHAR* Name, const T& DefaultValue, uint32 Flags)
{
	return CreateInPlaceDeprecatedVar<T>(DeprecationArgs, Name, Flags, 
		[&](const FString& DeprecationHelp, EConsoleVariableFlags ConsoleVariableFlags)
	{ 
		return new FConsoleVariable<T>(DefaultValue, *DeprecationHelp, ConsoleVariableFlags);
	});
}

template<typename T>
TAutoConsoleVariable<T>::TAutoConsoleVariable(const FCVarDeprecationArgs& DeprecationArgs, const TCHAR* Name, const T& DefaultValue, uint32 Flags)
	: TAutoConsoleVariable(CreateInPlaceDeprecatedVar(DeprecationArgs, Name, DefaultValue, Flags))
{
}

template CORE_API TAutoConsoleVariable<int32>::TAutoConsoleVariable(const FCVarDeprecationArgs& DeprecationArgs, const TCHAR* Name, const int32& DefaultValue, uint32 Flags);
template CORE_API TAutoConsoleVariable<float>::TAutoConsoleVariable(const FCVarDeprecationArgs& DeprecationArgs, const TCHAR* Name, const float& DefaultValue, uint32 Flags);
template CORE_API TAutoConsoleVariable<bool>::TAutoConsoleVariable(const FCVarDeprecationArgs& DeprecationArgs, const TCHAR* Name, const bool& DefaultValue, uint32 Flags);
template CORE_API TAutoConsoleVariable<FString>::TAutoConsoleVariable(const FCVarDeprecationArgs& DeprecationArgs, const TCHAR* Name, const FString& DefaultValue, uint32 Flags);

template<typename T, typename CmdType>
void CreateInPlaceDeprecatedCommand(const TCHAR* Name, const CmdType& Command, const TCHAR* DeprecatedAtVersion, EShadowCVarBehavior UsageBehavior, const TCHAR* DeprecationMessage)
{
#if !NO_CVARS
	check(Name != nullptr && FCString::Strlen(Name) > 0);
	check(DeprecatedAtVersion != nullptr && FCString::Strlen(DeprecatedAtVersion) > 0);

	// create a backing unregistered cvar with a dummy name, but this name won't show up since Unregistered
	FString BackingName = FString(Name) + "_dep";
	T* BackingCommand = new T(Command, *MakeDeprecatedHelp(Name, DeprecatedAtVersion, DeprecationMessage), (EConsoleVariableFlags)(ECVF_Default | ECVF_Unregistered));
	GetManager().AddConsoleObject(*BackingName, BackingCommand);

	// create the shadow cvar that shadows the backing cvar, so we can use all the existing logic
	FConsoleCommandDeprecatedNoReplace* DeprecatedCVar = new FConsoleCommandDeprecatedNoReplace(*BackingName, DeprecatedAtVersion, UsageBehavior, DeprecationMessage);
	// don't let the AddCosoleObject throw an error by calling GetFlags()
	GetManager().AddShadowConsoleObject(Name, DeprecatedCVar);

#endif
}

void CreateInPlaceDeprecatedCommandExec(const TCHAR* Name, const TCHAR* DeprecatedAtVersion, EShadowCVarBehavior UsageBehavior, const TCHAR* DeprecationMessage)
{
#if !NO_CVARS
	check(Name != nullptr && FCString::Strlen(Name) > 0);
	check(DeprecatedAtVersion != nullptr && FCString::Strlen(DeprecatedAtVersion) > 0);

	// create a backing unregistered cvar with a dummy name, but this name won't show up since Unregistered
	FString BackingName = FString(Name) + "_dep";
	FConsoleCommandExec* BackingCommand = new FConsoleCommandExec(*MakeDeprecatedHelp(Name, DeprecatedAtVersion, DeprecationMessage), (EConsoleVariableFlags)(ECVF_Default | ECVF_Unregistered));
	GetManager().AddConsoleObject(*BackingName, BackingCommand);

	// create the shadow cvar that shadows the backing cvar, so we can use all the existing logic
	FConsoleCommandDeprecatedNoReplace* DeprecatedCVar = new FConsoleCommandDeprecatedNoReplace(*BackingName, DeprecatedAtVersion, UsageBehavior, DeprecationMessage);
	// don't let the AddCosoleObject throw an error by calling GetFlags()
	GetManager().AddShadowConsoleObject(Name, DeprecatedCVar);

#endif
}

template<typename T>
TAutoConsoleVariableDeprecated_NoReplace<T>::TAutoConsoleVariableDeprecated_NoReplace(const TCHAR* Name, const T& DefaultValue, const TCHAR* DeprecatedAtVersion, EShadowCVarBehavior UsageBehavior, const TCHAR* DeprecationMessage)
{
	CreateInPlaceDeprecatedVar(FCVarDeprecationArgs{ DeprecatedAtVersion, UsageBehavior, DeprecationMessage }, Name, DefaultValue, ECVF_Default);
}

// because we create a variable whose type is not in the header, we have to instantiate each possible type here
template CORE_API TAutoConsoleVariableDeprecated_NoReplace<int32>::TAutoConsoleVariableDeprecated_NoReplace(const TCHAR* Name, const int32& DefaultValue, const TCHAR* DeprecatedAtVersion, EShadowCVarBehavior UsageBehavior, const TCHAR* DeprecationMessage);
template CORE_API TAutoConsoleVariableDeprecated_NoReplace<float>::TAutoConsoleVariableDeprecated_NoReplace(const TCHAR* Name, const float& DefaultValue, const TCHAR* DeprecatedAtVersion, EShadowCVarBehavior UsageBehavior, const TCHAR* DeprecationMessage);
template CORE_API TAutoConsoleVariableDeprecated_NoReplace<bool>::TAutoConsoleVariableDeprecated_NoReplace(const TCHAR* Name, const bool& DefaultValue, const TCHAR* DeprecatedAtVersion, EShadowCVarBehavior UsageBehavior, const TCHAR* DeprecationMessage);
template CORE_API TAutoConsoleVariableDeprecated_NoReplace<FString>::TAutoConsoleVariableDeprecated_NoReplace(const TCHAR* Name, const FString& DefaultValue, const TCHAR* DeprecatedAtVersion, EShadowCVarBehavior UsageBehavior, const TCHAR* DeprecationMessage);

FAutoConsoleCommandDeprecated_NoReplace::FAutoConsoleCommandDeprecated_NoReplace(const TCHAR* Name, const TCHAR* DeprecatedAtVersion, EShadowCVarBehavior UsageBehavior, const TCHAR* DeprecationMessage)
{
	CreateInPlaceDeprecatedCommandExec(Name, DeprecatedAtVersion, UsageBehavior, DeprecationMessage);
}

FAutoConsoleCommandDeprecated_NoReplace::FAutoConsoleCommandDeprecated_NoReplace(const TCHAR* Name, const FConsoleCommandDelegate& Command, const TCHAR* DeprecatedAtVersion, EShadowCVarBehavior UsageBehavior, const TCHAR* DeprecationMessage)
{
	CreateInPlaceDeprecatedCommand<FConsoleCommand>(Name, Command, DeprecatedAtVersion, UsageBehavior, DeprecationMessage);
}

FAutoConsoleCommandDeprecated_NoReplace::FAutoConsoleCommandDeprecated_NoReplace(const TCHAR* Name, const FConsoleCommandWithArgsDelegate& Command, const TCHAR* DeprecatedAtVersion, EShadowCVarBehavior UsageBehavior, const TCHAR* DeprecationMessage)
{
	CreateInPlaceDeprecatedCommand<FConsoleCommandWithArgs>(Name, Command, DeprecatedAtVersion, UsageBehavior, DeprecationMessage);
}

FAutoConsoleCommandDeprecated_NoReplace::FAutoConsoleCommandDeprecated_NoReplace(const TCHAR* Name, const FConsoleCommandWithOutputDeviceDelegate& Command, const TCHAR* DeprecatedAtVersion, EShadowCVarBehavior UsageBehavior, const TCHAR* DeprecationMessage)
{
	CreateInPlaceDeprecatedCommand<FConsoleCommandWithOutputDevice>(Name, Command, DeprecatedAtVersion, UsageBehavior, DeprecationMessage);
}

FAutoConsoleCommandDeprecated_NoReplace::FAutoConsoleCommandDeprecated_NoReplace(const TCHAR* Name, const FConsoleCommandWithArgsAndOutputDeviceDelegate& Command, const TCHAR* DeprecatedAtVersion, EShadowCVarBehavior UsageBehavior, const TCHAR* DeprecationMessage)
{
	CreateInPlaceDeprecatedCommand<FConsoleCommanWithArgsAndOutputDevice>(Name, Command, DeprecatedAtVersion, UsageBehavior, DeprecationMessage);
}

FAutoConsoleCommandDeprecated_NoReplace::FAutoConsoleCommandDeprecated_NoReplace(const TCHAR* Name, const FConsoleCommandWithWorldDelegate& Command, const TCHAR* DeprecatedAtVersion, EShadowCVarBehavior UsageBehavior, const TCHAR* DeprecationMessage)
{
	CreateInPlaceDeprecatedCommand<FConsoleCommandWithWorld>(Name, Command, DeprecatedAtVersion, UsageBehavior, DeprecationMessage);
}

FAutoConsoleCommandDeprecated_NoReplace::FAutoConsoleCommandDeprecated_NoReplace(const TCHAR* Name, const FConsoleCommandWithWorldArgsAndOutputDeviceDelegate& Command, const TCHAR* DeprecatedAtVersion, EShadowCVarBehavior UsageBehavior, const TCHAR* DeprecationMessage)
{
	CreateInPlaceDeprecatedCommand<FConsoleCommandWithWorldArgsAndOutputDevice>(Name, Command, DeprecatedAtVersion, UsageBehavior, DeprecationMessage);
}

FAutoConsoleCommandDeprecated_NoReplace::FAutoConsoleCommandDeprecated_NoReplace(const TCHAR* Name, const FConsoleCommandWithWorldAndArgsDelegate& Command, const TCHAR* DeprecatedAtVersion, EShadowCVarBehavior UsageBehavior, const TCHAR* DeprecationMessage)
{
	CreateInPlaceDeprecatedCommand<FConsoleCommandWithWorldAndArgs>(Name, Command, DeprecatedAtVersion, UsageBehavior, DeprecationMessage);
}





namespace ConsoleVariablePlatformMapping
{
	TMap<int, FName> ShaderPlatformToPlatformMap;
	TMap<FName, FName> PlatformToDeviceProfileMap;

	void RegisterShaderPlatformToPlatformMapping(int ShaderPlatform, FName PlatformName)
	{
		checkf(!ShaderPlatformToPlatformMap.Contains(ShaderPlatform) || ShaderPlatformToPlatformMap.FindRef(ShaderPlatform) == PlatformName, 
			TEXT("There's already a platform (%s) targeting ShaderPlatform %d, when setting mapping to %s"), 
			*ShaderPlatformToPlatformMap.FindRef(ShaderPlatform).ToString(), ShaderPlatform, *PlatformName.ToString());

		ShaderPlatformToPlatformMap.Add(ShaderPlatform, PlatformName);
	}

	void RegisterPlatformToDeviceProfileMapping(FName PlatformName, FName DeviceProfileName)
	{
		checkf(!PlatformToDeviceProfileMap.Contains(PlatformName) || PlatformToDeviceProfileMap.FindRef(PlatformName) == DeviceProfileName,
			TEXT("There's already a DP (%s) being used for platform %s, when setting mapping to %s"), 
			*PlatformToDeviceProfileMap.FindRef(PlatformName).ToString(), *PlatformName.ToString(), *DeviceProfileName.ToString());

		PlatformToDeviceProfileMap.Add(PlatformName, DeviceProfileName);
	}

	FName GetShaderPlatformToPlatformMapping(int ShaderPlatform)
	{
		// this will return NAME_None if it's not present
		return ShaderPlatformToPlatformMap.FindRef(ShaderPlatform);
	}

	FName GetPlatformToDeviceProfileMapping(FName PlatformName)
	{
		// this will return NAME_None if it's not present
		return PlatformToDeviceProfileMap.FindRef(PlatformName);
	}
}

#if !NO_CVARS

template<typename T>
IConsoleVariable* CreateInPlaceDeprecatedVarRef(const FCVarDeprecationArgs& DeprecationArgs, const TCHAR* Name, T& RefValue, uint32 Flags)
{
	return CreateInPlaceDeprecatedVar<T>(DeprecationArgs, Name, Flags, 
		[&](const FString& DeprecationHelp, EConsoleVariableFlags ConsoleVariableFlags)
	{ 
		return IConsoleManager::Get().RegisterConsoleVariableRef(Name, RefValue, *DeprecationHelp, ConsoleVariableFlags);
	});
}

FAutoConsoleVariable::FAutoConsoleVariable(uint32 Flags, const TCHAR* Name, bool DefaultValue, const TCHAR* Help)
	: FAutoConsoleObject(IConsoleManager::Get().RegisterConsoleVariable(Name, DefaultValue, Help, Flags))
{
}

FAutoConsoleVariable::FAutoConsoleVariable(uint32 Flags, const TCHAR* Name, int32 DefaultValue, const TCHAR* Help)
	: FAutoConsoleObject(IConsoleManager::Get().RegisterConsoleVariable(Name, DefaultValue, Help, Flags))
{
}

FAutoConsoleVariable::FAutoConsoleVariable(uint32 Flags, const TCHAR* Name, float DefaultValue, const TCHAR* Help)
	: FAutoConsoleObject(IConsoleManager::Get().RegisterConsoleVariable(Name, DefaultValue, Help, Flags))
{
}

FAutoConsoleVariable::FAutoConsoleVariable(uint32 Flags, const TCHAR* Name, const TCHAR* DefaultValue, const TCHAR* Help)
	: FAutoConsoleObject(IConsoleManager::Get().RegisterConsoleVariable(Name, DefaultValue, Help, Flags))
{
}

FAutoConsoleVariable::FAutoConsoleVariable(uint32 Flags, const TCHAR* Name, bool DefaultValue, const TCHAR* Help, const FConsoleVariableDelegate& Callback)
	: FAutoConsoleObject(IConsoleManager::Get().RegisterConsoleVariable(Name, DefaultValue, Help, Flags))
{
	AsVariable()->SetOnChangedCallback(Callback);
}

FAutoConsoleVariable::FAutoConsoleVariable(uint32 Flags, const TCHAR* Name, int32 DefaultValue, const TCHAR* Help, const FConsoleVariableDelegate& Callback)
	: FAutoConsoleObject(IConsoleManager::Get().RegisterConsoleVariable(Name, DefaultValue, Help, Flags))
{
	AsVariable()->SetOnChangedCallback(Callback);
}

FAutoConsoleVariable::FAutoConsoleVariable(uint32 Flags, const TCHAR* Name, float DefaultValue, const TCHAR* Help, const FConsoleVariableDelegate& Callback)
	: FAutoConsoleObject(IConsoleManager::Get().RegisterConsoleVariable(Name, DefaultValue, Help, Flags))
{
	AsVariable()->SetOnChangedCallback(Callback);
}

FAutoConsoleVariable::FAutoConsoleVariable(uint32 Flags, const TCHAR* Name, const TCHAR* DefaultValue, const TCHAR* Help, const FConsoleVariableDelegate& Callback)
	: FAutoConsoleObject(IConsoleManager::Get().RegisterConsoleVariable(Name, DefaultValue, Help, Flags))
{
	AsVariable()->SetOnChangedCallback(Callback);
}

FAutoConsoleVariableRef::FAutoConsoleVariableRef(uint32 Flags, const TCHAR* Name, int32& RefValue, const TCHAR* Help)
	: FAutoConsoleObject(IConsoleManager::Get().RegisterConsoleVariableRef(Name, RefValue, Help, Flags))
{
}

FAutoConsoleVariableRef::FAutoConsoleVariableRef(const FCVarDeprecationArgs& DeprecationArgs, const TCHAR* Name, int32& RefValue, uint32 Flags)
	: FAutoConsoleObject(CreateInPlaceDeprecatedVarRef(DeprecationArgs, Name, RefValue, Flags))
{
}

FAutoConsoleVariableRef::FAutoConsoleVariableRef(uint32 Flags, const TCHAR* Name, float& RefValue, const TCHAR* Help)
	: FAutoConsoleObject(IConsoleManager::Get().RegisterConsoleVariableRef(Name, RefValue, Help, Flags))
{
}

FAutoConsoleVariableRef::FAutoConsoleVariableRef(const FCVarDeprecationArgs& DeprecationArgs, const TCHAR* Name, float& RefValue, uint32 Flags)
	: FAutoConsoleObject(CreateInPlaceDeprecatedVarRef(DeprecationArgs, Name, RefValue, Flags))
{
}

FAutoConsoleVariableRef::FAutoConsoleVariableRef(uint32 Flags, const TCHAR* Name, bool& RefValue, const TCHAR* Help)
	: FAutoConsoleObject(IConsoleManager::Get().RegisterConsoleVariableRef(Name, RefValue, Help, Flags))
{
}

FAutoConsoleVariableRef::FAutoConsoleVariableRef(const FCVarDeprecationArgs& DeprecationArgs, const TCHAR* Name, bool& RefValue, uint32 Flags)
	: FAutoConsoleObject(CreateInPlaceDeprecatedVarRef(DeprecationArgs, Name, RefValue, Flags))
{
}

FAutoConsoleVariableRef::FAutoConsoleVariableRef(uint32 Flags, const TCHAR* Name, FString& RefValue, const TCHAR* Help)
	: FAutoConsoleObject(IConsoleManager::Get().RegisterConsoleVariableRef(Name, RefValue, Help, Flags))
{
}

FAutoConsoleVariableRef::FAutoConsoleVariableRef(const FCVarDeprecationArgs& DeprecationArgs, const TCHAR* Name, FString& RefValue, uint32 Flags)
	: FAutoConsoleObject(CreateInPlaceDeprecatedVarRef(DeprecationArgs, Name, RefValue, Flags))
{
}

FAutoConsoleVariableRef::FAutoConsoleVariableRef(uint32 Flags, const TCHAR* Name, FName& RefValue, const TCHAR* Help)
	: FAutoConsoleObject(IConsoleManager::Get().RegisterConsoleVariableRef(Name, RefValue, Help, Flags))
{
}

FAutoConsoleVariableRef::FAutoConsoleVariableRef(const FCVarDeprecationArgs& DeprecationArgs, const TCHAR* Name, FName& RefValue, uint32 Flags)
	: FAutoConsoleObject(CreateInPlaceDeprecatedVarRef(DeprecationArgs, Name, RefValue, Flags))
{
}

FAutoConsoleVariableRef::FAutoConsoleVariableRef(uint32 Flags, const TCHAR* Name, int32& RefValue, const TCHAR* Help, const FConsoleVariableDelegate& Callback)
	: FAutoConsoleObject(IConsoleManager::Get().RegisterConsoleVariableRef(Name, RefValue, Help, Flags))
{
	AsVariable()->SetOnChangedCallback(Callback);
}

FAutoConsoleVariableRef::FAutoConsoleVariableRef(const FCVarDeprecationArgs& DeprecationArgs, const TCHAR* Name, int32& RefValue, const FConsoleVariableDelegate& Callback, uint32 Flags)
	: FAutoConsoleObject(CreateInPlaceDeprecatedVarRef(DeprecationArgs, Name, RefValue, Flags))
{
	AsVariable()->SetOnChangedCallback(Callback);
}

FAutoConsoleVariableRef::FAutoConsoleVariableRef(uint32 Flags, const TCHAR* Name, float& RefValue, const TCHAR* Help, const FConsoleVariableDelegate& Callback)
	: FAutoConsoleObject(IConsoleManager::Get().RegisterConsoleVariableRef(Name, RefValue, Help, Flags))
{
	AsVariable()->SetOnChangedCallback(Callback);
}

FAutoConsoleVariableRef::FAutoConsoleVariableRef(const FCVarDeprecationArgs& DeprecationArgs, const TCHAR* Name, float& RefValue, const FConsoleVariableDelegate& Callback, uint32 Flags)
	: FAutoConsoleObject(CreateInPlaceDeprecatedVarRef(DeprecationArgs, Name, RefValue, Flags))
{
	AsVariable()->SetOnChangedCallback(Callback);
}

FAutoConsoleVariableRef::FAutoConsoleVariableRef(uint32 Flags, const TCHAR* Name, bool& RefValue, const TCHAR* Help, const FConsoleVariableDelegate& Callback)
	: FAutoConsoleObject(IConsoleManager::Get().RegisterConsoleVariableRef(Name, RefValue, Help, Flags))
{
	AsVariable()->SetOnChangedCallback(Callback);
}

FAutoConsoleVariableRef::FAutoConsoleVariableRef(const FCVarDeprecationArgs& DeprecationArgs, const TCHAR* Name, bool& RefValue, const FConsoleVariableDelegate& Callback, uint32 Flags)
	: FAutoConsoleObject(CreateInPlaceDeprecatedVarRef(DeprecationArgs, Name, RefValue, Flags))
{
	AsVariable()->SetOnChangedCallback(Callback);
}

FAutoConsoleVariableRef::FAutoConsoleVariableRef(uint32 Flags, const TCHAR* Name, FString& RefValue, const TCHAR* Help, const FConsoleVariableDelegate& Callback)
	: FAutoConsoleObject(IConsoleManager::Get().RegisterConsoleVariableRef(Name, RefValue, Help, Flags))
{
	AsVariable()->SetOnChangedCallback(Callback);
}

FAutoConsoleVariableRef::FAutoConsoleVariableRef(const FCVarDeprecationArgs& DeprecationArgs, const TCHAR* Name, FString& RefValue, const FConsoleVariableDelegate& Callback, uint32 Flags)
	: FAutoConsoleObject(CreateInPlaceDeprecatedVarRef(DeprecationArgs, Name, RefValue, Flags))
{
	AsVariable()->SetOnChangedCallback(Callback);
}

FAutoConsoleCommand::FAutoConsoleCommand(uint32 Flags, const TCHAR* Name, const TCHAR* Help, const FConsoleCommandDelegate& Command)
	: FAutoConsoleObject(IConsoleManager::Get().RegisterConsoleCommand(Name, Help, Command, Flags))
{
}

FAutoConsoleCommand::FAutoConsoleCommand(uint32 Flags, const TCHAR* Name, const TCHAR* Help, const FConsoleCommandWithArgsDelegate& Command)
	: FAutoConsoleObject(IConsoleManager::Get().RegisterConsoleCommand(Name, Help, Command, Flags))
{
}

FAutoConsoleCommand::FAutoConsoleCommand(uint32 Flags, const TCHAR* Name, const TCHAR* Help, const FConsoleCommandWithOutputDeviceDelegate& Command)
	: FAutoConsoleObject(IConsoleManager::Get().RegisterConsoleCommand(Name, Help, Command, Flags))
{
}

FAutoConsoleCommand::FAutoConsoleCommand(uint32 Flags, const TCHAR* Name, const TCHAR* Help, const FConsoleCommandWithArgsAndOutputDeviceDelegate& Command)
	: FAutoConsoleObject(IConsoleManager::Get().RegisterConsoleCommand(Name, Help, Command, Flags))
{
}

FAutoConsoleCommand::FAutoConsoleCommand(uint32 Flags, const TCHAR* Name, const TCHAR* Help, const FConsoleCommandWithWorldDelegate& Command)
	: FAutoConsoleObject(IConsoleManager::Get().RegisterConsoleCommand(Name, Help, Command, Flags))
{
}

FAutoConsoleCommand::FAutoConsoleCommand(uint32 Flags, const TCHAR* Name, const TCHAR* Help, const FConsoleCommandWithWorldArgsAndOutputDeviceDelegate& Command)
	: FAutoConsoleObject(IConsoleManager::Get().RegisterConsoleCommand(Name, Help, Command, Flags))
{
}

FAutoConsoleCommandWithWorld::FAutoConsoleCommandWithWorld(uint32 Flags, const TCHAR* Name, const TCHAR* Help, const FConsoleCommandWithWorldDelegate& Command)
	: FAutoConsoleObject(IConsoleManager::Get().RegisterConsoleCommand(Name, Help, Command, Flags))
{
}

FAutoConsoleCommandWithWorldAndArgs::FAutoConsoleCommandWithWorldAndArgs(uint32 Flags, const TCHAR* Name, const TCHAR* Help, const FConsoleCommandWithWorldAndArgsDelegate& Command)
	: FAutoConsoleObject(IConsoleManager::Get().RegisterConsoleCommand(Name, Help, Command, Flags))
{
}

FAutoConsoleCommandWithArgsAndOutputDevice::FAutoConsoleCommandWithArgsAndOutputDevice(uint32 Flags, const TCHAR* Name, const TCHAR* Help, const FConsoleCommandWithArgsAndOutputDeviceDelegate& Command)
	: FAutoConsoleObject(IConsoleManager::Get().RegisterConsoleCommand(Name, Help, Command, Flags))
{
}

FAutoConsoleCommandWithOutputDevice::FAutoConsoleCommandWithOutputDevice(uint32 Flags, const TCHAR* Name, const TCHAR* Help, const FConsoleCommandWithOutputDeviceDelegate& Command)
	: FAutoConsoleObject(IConsoleManager::Get().RegisterConsoleCommand(Name, Help, Command, Flags))
{
}

FAutoConsoleCommandWithWorldArgsAndOutputDevice::FAutoConsoleCommandWithWorldArgsAndOutputDevice(uint32 Flags, const TCHAR* Name, const TCHAR* Help, const FConsoleCommandWithWorldArgsAndOutputDeviceDelegate& Command)
	: FAutoConsoleObject(IConsoleManager::Get().RegisterConsoleCommand(Name, Help, Command, Flags))
{
}

#endif

#undef LOCTEXT_NAMESPACE
