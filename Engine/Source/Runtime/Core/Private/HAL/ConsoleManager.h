// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/IConsoleManager.h"

class FConsoleManager : public IConsoleManager
{
public:

	FConsoleManager();
	~FConsoleManager();

	virtual IConsoleVariable* RegisterConsoleVariable(const TCHAR* Name, bool DefaultValue, const TCHAR* Help, uint32 Flags) override;
	virtual IConsoleVariable* RegisterConsoleVariable(const TCHAR* Name, int32 DefaultValue, const TCHAR* Help, uint32 Flags) override;
	virtual IConsoleVariable* RegisterConsoleVariable(const TCHAR* Name, float DefaultValue, const TCHAR* Help, uint32 Flags) override;
	virtual IConsoleVariable* RegisterConsoleVariable(const TCHAR* Name, const TCHAR* DefaultValue, const TCHAR* Help, uint32 Flags) override;
	virtual IConsoleVariable* RegisterConsoleVariable(const TCHAR* Name, const FString& DefaultValue, const TCHAR* Help, uint32 Flags) override;

	virtual IConsoleVariable* RegisterDelegatedConsoleVariable(const TCHAR* Name, int32 DefaultValue, TFunction<int32()> Getter, TFunction<void(const int32&)> Setter, const TCHAR* Help, uint32 Flags) override;
	virtual IConsoleVariable* RegisterDelegatedConsoleVariable(const TCHAR* Name, float DefaultValue, TFunction<float()> Getter, TFunction<void(const float&)> Setter, const TCHAR* Help, uint32 Flags) override;
	virtual IConsoleVariable* RegisterDelegatedConsoleVariable(const TCHAR* Name, bool DefaultValue, TFunction<bool()> Getter, TFunction<void(const bool&)> Setter, const TCHAR* Help, uint32 Flags) override;
	virtual IConsoleVariable* RegisterDelegatedConsoleVariable(const TCHAR* Name, const FString& DefaultValue, TFunction<FString()> Getter, TFunction<void(const FString&)> Setter, const TCHAR* Help, uint32 Flags) override;

	virtual IConsoleVariable* RegisterConsoleVariableRef(const TCHAR* Name, bool& RefValue, const TCHAR* Help, uint32 Flags) override;
	virtual IConsoleVariable* RegisterConsoleVariableRef(const TCHAR* Name, int32& RefValue, const TCHAR* Help, uint32 Flags) override;
	virtual IConsoleVariable* RegisterConsoleVariableRef(const TCHAR* Name, float& RefValue, const TCHAR* Help, uint32 Flags) override;
	virtual IConsoleVariable* RegisterConsoleVariableRef(const TCHAR* Name, FString& RefValue, const TCHAR* Help, uint32 Flags) override;
	virtual IConsoleVariable* RegisterConsoleVariableRef(const TCHAR* Name, FName& RefValue, const TCHAR* Help, uint32 Flags) override;
	virtual IConsoleVariable* RegisterConsoleVariableBitRef(const TCHAR* CVarName, const TCHAR* FlagName, uint32 BitNumber, uint8* Force0MaskPtr, uint8* Force1MaskPtr, const TCHAR* Help, uint32 Flags) override;

	virtual IConsoleCommand* RegisterConsoleCommand(const TCHAR* Name, const TCHAR* Help, const FConsoleCommandDelegate& Command, uint32 Flags) override;
	virtual IConsoleCommand* RegisterConsoleCommand(const TCHAR* Name, const TCHAR* Help, const FConsoleCommandWithArgsDelegate& Command, uint32 Flags) override;
	virtual IConsoleCommand* RegisterConsoleCommand(const TCHAR* Name, const TCHAR* Help, const FConsoleCommandWithWorldDelegate& Command, uint32 Flags) override;
	virtual IConsoleCommand* RegisterConsoleCommand(const TCHAR* Name, const TCHAR* Help, const FConsoleCommandWithWorldAndArgsDelegate& Command, uint32 Flags) override;
	virtual IConsoleCommand* RegisterConsoleCommand(const TCHAR* Name, const TCHAR* Help, const FConsoleCommandWithArgsAndOutputDeviceDelegate& Command, uint32 Flags) override;
	virtual IConsoleCommand* RegisterConsoleCommand(const TCHAR* Name, const TCHAR* Help, const FConsoleCommandWithWorldArgsAndOutputDeviceDelegate& Command, uint32 Flags) override;
	virtual IConsoleCommand* RegisterConsoleCommand(const TCHAR* Name, const TCHAR* Help, const FConsoleCommandWithOutputDeviceDelegate& Command, uint32 Flags) override;
	virtual IConsoleCommand* RegisterConsoleCommand(const TCHAR* Name, const TCHAR* Help, uint32 Flags) override;

	virtual FConsoleVariableMulticastDelegate& OnCVarUnregistered()override;
	virtual FConsoleObjectWithNameMulticastDelegate& OnConsoleObjectUnregistered() override;

	virtual FString FindConsoleObjectName(const IConsoleObject* Obj) const override;
	virtual IConsoleObject* FindConsoleObject(const TCHAR* Name, bool bTrackFrequentCalls = true) const override;
	virtual IConsoleVariable* FindConsoleVariable(const TCHAR* Name, bool bTrackFrequentCalls = true) const override;

	virtual FConsoleVariableSinkHandle RegisterConsoleVariableSink_Handle(const FConsoleCommandDelegate& Command) override;
	virtual void UnregisterConsoleVariableSink_Handle(FConsoleVariableSinkHandle Handle) override;
	virtual void CallAllConsoleVariableSinks() override;

	virtual void ForEachConsoleObjectThatStartsWith(const FConsoleObjectVisitor& Visitor, const TCHAR* ThatStartsWith) const override;
	virtual void ForEachConsoleObjectThatContains(const FConsoleObjectVisitor& Visitor, const TCHAR* ThatContains) const override;
	virtual bool ProcessUserConsoleInput(const TCHAR* InInput, FOutputDevice& Ar, UWorld* InWorld) override;
	virtual void AddConsoleHistoryEntry(const TCHAR* Key, const TCHAR* Input) override;
	virtual void GetConsoleHistory(const TCHAR* Key, TArray<FString>& Out) override;
	virtual bool IsNameRegistered(const TCHAR* Name) const override;
	virtual void RegisterThreadPropagation(uint32 ThreadId, IConsoleThreadPropagation* InCallback) override;
	virtual void UnregisterConsoleObject(IConsoleObject* Object, bool bKeepState) override;
	virtual void UnsetAllConsoleVariablesWithTag(FName Tag, EConsoleVariableFlags Priority) override;
	virtual void BatchUpdateTag(FName Tag, const TMap<FName, FString>& CVarsAndValues) override;

#if ALLOW_OTHER_PLATFORM_CONFIG
	virtual void LoadAllPlatformCVars(FName PlatformName, const FString& DeviceProfileName = FString()) override;
	virtual void ClearAllPlatformCVars(FName PlatformName = NAME_None, const FString& DeviceProfileName = FString()) override;
	virtual void PreviewPlatformCVars(FName PlatformName, const FString& DeviceProfileName, FName PreviewModeTag) override;
	virtual void StompPlatformCVars(FName PlatformName, const FString& DeviceProfileName, FName Tag, EConsoleVariableFlags SetBy, EConsoleVariableFlags RequiredFlags, EConsoleVariableFlags DisallowedFlags) override;
#endif

	/**
	 * @param Name must not be 0, must not be empty
	 * @param Obj must not be 0
	 * @return 0 if the name was already in use
	 */
	 IConsoleObject* AddConsoleObject(const TCHAR* Name, IConsoleObject* Obj);

	/**
	 * Similar to AddConsoleObject, but it just adds it without any flag checking or preexisting var checking
	 */
	void AddShadowConsoleObject(const TCHAR* Name, IConsoleObject* Obj);

	/** Internally needed for ECVF_RenderThreadSafe */
	IConsoleThreadPropagation* GetThreadPropagationCallback();

	/** Internally needed for ECVF_RenderThreadSafe */
	bool IsThreadPropagationThread();

	void OnCVarChanged();
	void DumpObjects(const TCHAR* Params, FOutputDevice& InAr, bool bDisplayCommands) const;

#if ALLOW_OTHER_PLATFORM_CONFIG
	void OnCreatedPlatformCVar(IConsoleVariable* MainVariable, TSharedPtr<IConsoleVariable> PlatformVariable, FName PlatformKey);
#endif

private:

	template <class T>
	IConsoleVariable* RegisterDelegatedConsoleVariableImpl(
		const TCHAR* Name, T DefaultValue,
		TFunction<T()> Getter, TFunction<void(const T&)> Setter,
		const TCHAR* Help, uint32 Flags);

	/**
	 * @param Stream must not be 0
	 * @param Pattern must not be 0
	 */
	static bool MatchPartialName(const TCHAR* Stream, const TCHAR* Pattern);

	/** Returns true if Pattern is found in Stream, case insensitive. */
	static bool MatchSubstring(const TCHAR* Stream, const TCHAR* Pattern);

	/**
	 * Get string till whitespace, jump over whitespace
	 * inefficient but this code is not performance critical
	 */
	static FString GetTextSection(const TCHAR*& It);

	/** same as FindConsoleObject() but ECVF_CreatedFromIni are not filtered out (for internal use) */
	IConsoleObject* FindConsoleObjectUnfiltered(const TCHAR* Name) const;

	/**
	 * Unregisters a console variable or command, if that object was registered. For console variables, this will
	 * actually only "deactivate" the variable so if it becomes registered again the state may persist (unless
	 * bKeepState is false).
	 *
	 * @param Name Name of the console object to remove (not case sensitive)
	 * @param bKeepState if the current state is kept in memory until a cvar with the same name is registered
	 */
	void UnregisterConsoleObject(const TCHAR* Name, bool bKeepState);

	/** Reads HistoryEntriesMap from the .ini file (if not already loaded) */
	void LoadHistoryIfNeeded();

	/** Writes HistoryEntriesMap to the .ini file */
	void SaveHistory();

	/**
	 * Map of console variables and commands, indexed by the name of that command or variable
	 * [name] = pointer (pointer must not be 0)
	 */
	TMap<FString, IConsoleObject*> ConsoleObjects;

	bool bHistoryWasLoaded;
	TMap<FString, TArray<FString>> HistoryEntriesMap;
	TArray<FConsoleCommandDelegate> ConsoleVariableChangeSinks;
	FConsoleVariableMulticastDelegate ConsoleVariableUnregisteredDelegate;
	FConsoleObjectWithNameMulticastDelegate ConsoleObjectUnregisteredDelegate;
	IConsoleThreadPropagation* ThreadPropagationCallback;
	FCriticalSection CachedPlatformsAndDeviceProfilesLock;
	TSet<FName> CachedPlatformsAndDeviceProfiles;
	TMultiMap<FString, TTuple<FString, EConsoleVariableFlags>> UnknownCVarCache;

	/** If true the next call to CallAllConsoleVariableSinks() we will call all registered sinks */
	bool bCallAllConsoleVariableSinks;

	/**
	 * Used to prevent concurrent access to ConsoleObjects. We don't aim to solve all concurrency problems (for example
	 * registering and unregistering a cvar on different threads, or reading a cvar from one thread while writing it
	 * from a different thread). Rather we just ensure that operations on a cvar from one thread will not conflict with
	 * operations on another cvar from another thread.
	 */
	mutable FTransactionallySafeCriticalSection ConsoleObjectsSynchronizationObject;
};
