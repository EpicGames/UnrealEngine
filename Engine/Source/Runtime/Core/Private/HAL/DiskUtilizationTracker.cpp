// Copyright Epic Games, Inc. All Rights Reserved.

#include "HAL/DiskUtilizationTracker.h"

#if TRACK_DISK_UTILIZATION
#include "ProfilingDebugging/CsvProfiler.h"
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformTime.h"
#include "Misc/CommandLine.h"
#include "Misc/Parse.h"

CSV_DEFINE_CATEGORY(DiskIO, true); 

DEFINE_LOG_CATEGORY_STATIC(LogDiskIO, Log, All);

static FAutoConsoleCommand GDumpShortTermIOStats(
	TEXT("disk.DumpShortTermStats"),
	TEXT("Dumps short term disk I/O stats."),
	FConsoleCommandDelegate::CreateLambda([]
	{
		UE_LOGF(LogDiskIO, Display, "Disk I/O short term stats:");
		GDiskUtilizationTracker.GetShortTermStats().Dump();
	})
);

void FDiskUtilizationTracker::UtilizationStats::Dump() const
{
	UE_LOGF(LogDiskIO, Display, "Total Reads: %llu Total Bytes Read: %llu", TotalReads, TotalBytesRead);
	UE_LOGF(LogDiskIO, Display, "Total IO Time: %fs Total Idle Time: %fs", TotalIOTime, TotalIdleTime);
	UE_LOGF(LogDiskIO, Display, "Read Throughput: %fMB/s Pct Time Idle: %f%%", GetReadThrougputMBS(), GetPercentTimeIdle());
}

float FDiskUtilizationTracker::GetThrottleRateMBS()
{
	float ThrottledThroughputMBs = 0.0f;
	FParse::Value(FCommandLine::Get(), TEXT("-ThrottleDiskIOMBS="), ThrottledThroughputMBs);
	if (ThrottledThroughputMBs > 0.0f)
	{
		UE_LOGF(LogDiskIO, Warning, "Disk IO will be throttled to %fMB/s", ThrottledThroughputMBs);
	}

	return ThrottledThroughputMBs;
}

void FDiskUtilizationTracker::StartRead(uint64 InReadBytes, uint64 InSeekDistance)
{
	static bool bBreak = false;

	bool bReset = bResetShortTermStats.AtomicSet(false);

	if (bReset)
	{
		ShortTermStats.Reset();
		UE_LOGF(LogDiskIO, Display, "Short term stats reset");
		bBreak = true;
	}

	// update total reads
	LongTermStats.TotalReads++;
	ShortTermStats.TotalReads++;

	// update seek data
	if (InSeekDistance > 0)
	{
		LongTermStats.TotalSeeks++;
		ShortTermStats.TotalSeeks++;

		LongTermStats.TotalSeekDistance += InSeekDistance;
		ShortTermStats.TotalSeekDistance += InSeekDistance;
	}

	{
		FScopeLock Lock(&CriticalSection);

		if (InFlightReads == 0)
		{
			// if this is the first started read from idle start 
			ReadStartCycle = FPlatformTime::Cycles64();

			// update idle time (if we've been idle)
			if (IdleStartCycle > 0)
			{
				const double IdleTime = double(ReadStartCycle - IdleStartCycle) * FPlatformTime::GetSecondsPerCycle64();

				LongTermStats.TotalIdleTime += IdleTime;
				ShortTermStats.TotalIdleTime += bReset ? 0 : IdleTime;

				CSV_CUSTOM_STAT(DiskIO, AccumulatedIdleTime, float(IdleTime), ECsvCustomStatOp::Accumulate);
			}
		}

		InFlightBytes += InReadBytes;
		InFlightReads++;
	}
}

void FDiskUtilizationTracker::FinishRead()
{
	// if we're the last in flight read update the start idle counter
	{
		FScopeLock Lock(&CriticalSection);
		check(InFlightReads > 0);

		if (--InFlightReads == 0)
		{
#if !UE_BUILD_SHIPPING
			static const float ThrottledThroughputBS = GetThrottleRateMBS() * 1024 * 1024;

			if ((ThrottledThroughputBS > 0.0f) && (LongTermStats.GetReadThrougputBS() > ThrottledThroughputBS))
			{
				const double IOTime = double(FPlatformTime::Cycles64() - ReadStartCycle) * FPlatformTime::GetSecondsPerCycle64();
				const double ThrottledIOTime = (double(LongTermStats.TotalBytesRead + InFlightBytes) / ThrottledThroughputBS) - LongTermStats.TotalIOTime;

				if (IOTime < ThrottledIOTime)
				{
					FPlatformProcess::Sleep(float(ThrottledIOTime - IOTime));
				}
			}
#endif // !UE_BUILD_SHIPPING

			IdleStartCycle = FPlatformTime::Cycles64();

			// update our read counters
			const double IOTime = double(IdleStartCycle - ReadStartCycle) * FPlatformTime::GetSecondsPerCycle64();

			LongTermStats.TotalIOTime += IOTime;
			ShortTermStats.TotalIOTime += IOTime;

			LongTermStats.TotalBytesRead += InFlightBytes;
			ShortTermStats.TotalBytesRead += InFlightBytes;

			CSV_CUSTOM_STAT(DiskIO, AccumulatedIOTime, float(IOTime), ECsvCustomStatOp::Accumulate);

			InFlightBytes = 0;
		}

	}
	MaybePrint();
}

void FDiskUtilizationTracker::MaybePrint()
{
#if !UE_BUILD_SHIPPING && SPEW_DISK_UTILIZATION
	static double LastPrintSeconds = 0.0;

	double CurrentSeconds = FPlatformTime::Seconds();

	// if we haven't printed or haven't in a while and there's been some I/O emit stats
	if (((LastPrintSeconds == 0.0) || ((CurrentSeconds - LastPrintSeconds) > PrintFrequencySeconds)) && (TotalIOTime > 0.0))
	{
		{
			// emit recent I/O info
			static uint64 LastReads = 0;
			static uint64 LastBytesRead = 0;

			static double LastIOTime = 0.0;
			static double LastIdleTime = 0.0;

			static uint32 LastSeeks = 0;
			static uint64 LastSeekDistance = 0;

			if ((LastPrintSeconds > 0.0) && (TotalBytesRead > LastBytesRead))
			{
				float TimeInterval = CurrentSeconds - LastPrintSeconds;

				double RecentIOTime = TotalIOTime - LastIOTime;
				double RecentIdleTime = TotalIdleTime - LastIdleTime;

				float Utilization = float(100.0 * RecentIOTime / (RecentIOTime + RecentIdleTime));

				uint64 RecentBytesRead = TotalBytesRead - LastBytesRead;

				double OverallThroughput = double(RecentBytesRead) / (RecentIOTime + RecentIdleTime) / (1024 * 1024);
				double ReadThroughput = double(RecentBytesRead) / RecentIOTime / (1024 * 1024);

				uint32 RecentSeeks = NumSeeks - LastSeeks;
				uint64 RecentSeekDistance = TotalSeekDistance - LastSeekDistance;

				double KBPerSeek = RecentSeeks ? double(RecentBytesRead) / (1024 * RecentSeeks) : 0;
				double AvgSeek = RecentSeeks ? double(RecentSeekDistance) / double(RecentSeeks) : 0;

				uint64 RecentReads = NumReads - LastReads;

				FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Recent Disk Utilization: %5.2f%% over %6.2fs\t%.2f MB/s\t%.2f Actual MB/s\t(%d Reads, %d Seeks, %.2f kbytes / seek, %.2f ave seek)\r\n"), 
					Utilization, TimeInterval, OverallThroughput, ReadThroughput, RecentReads, RecentSeeks, KBPerSeek, AvgSeek);
			}

			LastReads = NumReads;
			LastBytesRead = TotalBytesRead;

			LastIOTime = TotalIOTime;
			LastIdleTime = TotalIOTime;

			LastSeeks = NumSeeks;
			LastSeekDistance = TotalSeekDistance;
		}

		{
			// emit recent I/O info
			float Utilization = float(100.0 * TotalIOTime / (TotalIOTime + TotalIdleTime));

			double OverallThroughput = GetOverallThroughputMBS();
			double ReadThroughput = GetReadThrougputMBS();

			double KBPerSeek = NumSeeks ? double(TotalBytesRead) / (1024 * NumSeeks) : 0;
			double AvgSeek = NumSeeks ? double(TotalSeekDistance) / double(NumSeeks) : 0;

			FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Overall Disk Utilization: %5.2f%%\t%.2f MB/s\t%.2f Actual MB/s\t(%d Reads, %d Seeks, %.2f kbytes / seek, %.2f ave seek)\r\n"),
				Utilization, OverallThroughput, ReadThroughput, NumReads, NumSeeks, KBPerSeek, AvgSeek);
		}
	}

	LastPrintSeconds = CurrentSeconds;

#endif //!UE_BUILD_SHIPPING && SPEW_DISK_UTILIZATION
}

struct FDiskUtilizationTracker GDiskUtilizationTracker;
#endif // TRACK_DISK_UTILIZATION