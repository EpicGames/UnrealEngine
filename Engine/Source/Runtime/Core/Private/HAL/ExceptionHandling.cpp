// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ExceptionHandling.cpp: Exception handling for functions that want to create crash dumps.
=============================================================================*/

#include "HAL/ExceptionHandling.h"
#include "Templates/UnrealTemplate.h"
#include "Containers/UnrealString.h"
#include "Logging/LogMacros.h"
#include "CoreGlobals.h"
#include "Misc/CoreDelegates.h"
#include "Misc/OutputDeviceRedirector.h"

#ifndef UE_ASSERT_ON_BUILD_INTEGRITY_COMPROMISED
#define UE_ASSERT_ON_BUILD_INTEGRITY_COMPROMISED 0
#endif

/** Whether we should generate crash reports even if the debugger is attached. */
/** NOTE: If -nocrashreports is passed in, this will be ignored. */
CORE_API bool GAlwaysReportCrash = false;

/** Whether to use ClientReportClient rather than the old AutoReporter. */
CORE_API bool GUseCrashReportClient = true;

CORE_API TCHAR MiniDumpFilenameW[1024] = {};


bool GEnsureShowsCRC = false;

void ReportInteractiveEnsure(const TCHAR* InMessage)
{
	GEnsureShowsCRC = true;

#if PLATFORM_USE_REPORT_ENSURE
	GLog->FlushThreadedLogs();
	ReportEnsure(InMessage, nullptr);
#endif

	GEnsureShowsCRC = false;
}

bool IsInteractiveEnsureMode()
{
	return GEnsureShowsCRC;
}