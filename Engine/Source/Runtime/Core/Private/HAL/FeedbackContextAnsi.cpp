// Copyright Epic Games, Inc. All Rights Reserved.

#include "HAL/FeedbackContextAnsi.h"

#include "Logging/StructuredLog.h"

void FFeedbackContextAnsi::LocalPrint(const TCHAR* Str)
{
#if PLATFORM_APPLE || PLATFORM_UNIX
	printf("%s", (const char*)StringCast<UTF8CHAR>(Str).Get());
#elif PLATFORM_MICROSOFT
	wprintf(TEXT("%ls"), Str);
#else
	// If this function ever gets more complicated, we could make a PlatformMisc::Printf, and each platform can then 
	// do the right thing. For instance, LocalPrint is OutputDebugString on Windows, which messes up a lot of stuff
	FPlatformMisc::LocalPrint(Str);
#endif
	fflush(stdout);
}

bool FFeedbackContextAnsi::IsUsingLocalPrint() const
{
	// When -stdout is specified then FOutputDeviceStdOutput will be installed and pipe logging to stdout.
	// Skip calling LocalPrint in that case or else duplicate messages will be written to stdout.
	// A similar issue happens when a Console is shown.
	static bool bIsUsingLocalPrint =
		!FParse::Param(FCommandLine::Get(), TEXT("stdout")) &&
		(!GLogConsole || !GLogConsole->IsShown());
	return bIsUsingLocalPrint;
}

void FFeedbackContextAnsi::Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const FName& Category)
{
	Serialize(V, Verbosity, Category, -1.0);
}

void FFeedbackContextAnsi::Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const FName& Category, double Time)
{
	if (IsUsingLocalPrint() && (Verbosity == ELogVerbosity::Error || Verbosity == ELogVerbosity::Warning || Verbosity == ELogVerbosity::Display))
	{
		TStringBuilder<512> Line;
		FormatLine(Line, V, Verbosity, Category, Time);
		Line.AppendChar(TEXT('\n'));
		LocalPrint(*Line);
	}

	FFeedbackContext::Serialize(V, Verbosity, Category, Time);

	if (AuxOut)
	{
		AuxOut->Serialize(V, Verbosity, Category, Time);
	}
}

void FFeedbackContextAnsi::SerializeRecord(const UE::FLogRecord& Record)
{
	const ELogVerbosity::Type Verbosity = Record.GetVerbosity();
	if (IsUsingLocalPrint() && (Verbosity == ELogVerbosity::Error || Verbosity == ELogVerbosity::Warning || Verbosity == ELogVerbosity::Display))
	{
		TStringBuilder<512> Line;
		FormatRecordLine(Line, Record);
		Line.AppendChar(TEXT('\n'));
		LocalPrint(*Line);
	}

	FFeedbackContext::SerializeRecord(Record);

	if (AuxOut)
	{
		AuxOut->SerializeRecord(Record);
	}
}

bool FFeedbackContextAnsi::YesNof(const FText& Question)
{
	if (GIsClient || GIsEditor)
	{
		LocalPrint(*Question.ToString());
		LocalPrint(TEXT(" (Y/N): "));
		if (GIsSilent || FApp::IsUnattended())
		{
			LocalPrint(TEXT("Y\n"));
			return true;
		}
		else
		{
			char InputText[256];
			if (fgets(InputText, sizeof(InputText), stdin) != nullptr)
			{
				return InputText[0] == 'Y' || InputText[0] == 'y';
			}
		}
	}
	return true;
}
