// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	FileManagerGeneric.cpp: Unreal generic file manager support code.

	This base class simplifies IFileManager implementations by providing
	simple, unoptimized implementations of functions whose implementations
	can be derived from other functions.

=============================================================================*/

#include "HAL/FileManagerGeneric.h"
#include "Logging/LogMacros.h"
#include "Async/Mutex.h"
#include "Misc/AccessDetection.h"
#include "Misc/Parse.h"
#include "Misc/CommandLine.h"
#include "Misc/Paths.h"
#include "Misc/SecureHash.h"
#include "Misc/ScopeLock.h"
#include "Templates/UniquePtr.h"
#include <time.h>

DEFINE_LOG_CATEGORY_STATIC( LogFileManager, Log, All );

#define COPYBLOCKSIZE	32768

/*-----------------------------------------------------------------------------
	File Manager.
-----------------------------------------------------------------------------*/

void FFileManagerGeneric::ProcessCommandLineOptions() 
{
#if !UE_BUILD_SHIPPING
	if( FParse::Param( FCommandLine::Get(),TEXT( "CLEANSCREENSHOTS" ) ) )
	{
		DeleteDirectory( *FPaths::ScreenShotDir(), false, true );
	}

	if( FParse::Param( FCommandLine::Get(),TEXT( "CLEANLOGS" ) ) )
	{
		DeleteDirectory( *FPaths::ProjectLogDir(), false, true );
	}
#endif
}

FArchive* FFileManagerGeneric::CreateFileReaderInternal( const TCHAR* InFilename, uint32 Flags, uint32 BufferSize )
{
	IFileHandle* Handle = GetLowLevel().OpenRead( InFilename, !!(Flags & FILEREAD_AllowWrite) );
	if( !Handle )
	{
		if( Flags & FILEREAD_NoFail )
		{
			UE_LOGF( LogFileManager, Fatal, "Failed to read file: %ls", InFilename );
		}
		return NULL;
	}
	return new FArchiveFileReaderGeneric( Handle, InFilename, Handle->Size(), BufferSize, Flags );
}

/**
* Dummy archive that doesn't actually write anything
* it just updates the file pos when seeking
*/
class FArchiveFileWriterDummy final : public FArchive
{
public:
	FArchiveFileWriterDummy()
		:	Pos( 0 )
	{
		this->SetIsSaving(true);
		this->SetIsPersistent(true);
	}
	virtual ~FArchiveFileWriterDummy()
	{
		Close();
	}
	virtual void Seek( int64 InPos ) override
	{
		Pos = InPos;
	}
	virtual int64 Tell() override
	{
		return Pos;
	}
	virtual void Serialize( void* V, int64 Length ) override
	{
		Pos += Length;
	}
	virtual FString GetArchiveName() const override { return TEXT( "FArchiveFileWriterDummy" ); }
protected:
	int64             Pos;
};

FArchive* FFileManagerGeneric::CreateFileWriterInternal( const TCHAR* Filename, uint32 Flags, uint32 BufferSize )
{
	// Only allow writes to files that are not signed 
	// Except if the file is missing( that way corrupt ini files can be autogenerated by deleting them )
	if( FSHA1::GetFileSHAHash( Filename, NULL ) && FileSize( Filename ) != -1 )
	{
		UE_LOGF( LogFileManager, Log, "Can't write to signed game file: %ls",Filename );
		return new FArchiveFileWriterDummy();
	}

	if( Flags & FILEWRITE_EvenIfReadOnly )
	{
		GetLowLevel().SetReadOnly( Filename, false );
	}

	IFileHandle* Handle = GetLowLevel().OpenWrite( Filename, !!( Flags & FILEWRITE_Append ), !!( Flags & FILEWRITE_AllowRead ) );

	if (!Handle)
	{
		MakeDirectory(*FPaths::GetPath(Filename), true);
		
		Handle = GetLowLevel().OpenWrite(Filename, !!(Flags & FILEWRITE_Append), !!(Flags & FILEWRITE_AllowRead));
	}

	if( !Handle )
	{
		if( Flags & FILEWRITE_NoFail )
		{
			UE_LOGF( LogFileManager, Fatal, "Failed to create file: %ls", Filename );
		}
		return NULL;
	}
	return new FArchiveFileWriterGeneric(Handle, Filename, Handle->Tell(), BufferSize, Flags);
}


int64 FFileManagerGeneric::FileSize( const TCHAR* Filename )
{
	return GetLowLevel().FileSize( Filename );
}

uint32 FFileManagerGeneric::Copy( const TCHAR* Dest, const TCHAR* Src, bool Replace, bool EvenIfReadOnly, bool Attributes, FCopyProgress* Progress, EFileRead ReadFlags, EFileWrite WriteFlags)
{
	uint32	Result = COPY_OK;
	if( FPaths::ConvertRelativePathToFull(Dest) == FPaths::ConvertRelativePathToFull(Src) )
	{
		Result = COPY_Fail;
	}
	else if( Progress )
	{
		Result = CopyWithProgress(Dest, Src, Replace, EvenIfReadOnly, Attributes, Progress, ReadFlags, WriteFlags);
	}
	else if( !Replace && GetLowLevel().FileExists(Dest) )
	{
		Result = COPY_Fail;
	}
	else
	{
		if( EvenIfReadOnly )
		{
			GetLowLevel().SetReadOnly(Dest, false );
		}
		MakeDirectory( *FPaths::GetPath(Dest), true );
		EPlatformFileRead PlatformFileRead = (ReadFlags & FILEREAD_AllowWrite) ? EPlatformFileRead::AllowWrite : EPlatformFileRead::None;
		EPlatformFileWrite PlatformFileWrite = (WriteFlags & FILEWRITE_AllowRead) ? EPlatformFileWrite::AllowRead : EPlatformFileWrite::None;
		if( !GetLowLevel().CopyFile(Dest, Src, PlatformFileRead, PlatformFileWrite) )
		{
			Result = COPY_Fail;
		}
	}

	// Restore read-only attribute if required
	if( Result == COPY_OK && Attributes )
	{
		GetLowLevel().SetReadOnly(Dest, GetLowLevel().IsReadOnly(Src) );
	}

	return Result;
}

uint32 FFileManagerGeneric::CopyWithProgress(const TCHAR* InDestFile, const TCHAR* InSrcFile, bool ReplaceExisting, bool EvenIfReadOnly, bool Attributes, FCopyProgress* Progress, EFileRead ReadFlags, EFileWrite WriteFlags)
{
	uint32	Result = COPY_OK;

	// Direct file copier.
	if( Progress->Poll( 0.0 ) )
	{
		FString SrcFile		= InSrcFile;
		FString DestFile	= InDestFile;
	
		FArchive* Src = CreateFileReader( *SrcFile, ReadFlags );
		if( !Src )
		{
			Result = COPY_Fail;
		}
		else
		{
			FArchive* Dest = CreateFileWriter( *DestFile, ( ReplaceExisting ? 0 : FILEWRITE_NoReplaceExisting ) | ( EvenIfReadOnly ? FILEWRITE_EvenIfReadOnly : 0 ) | WriteFlags );
			if( !Dest )
			{
				Result = COPY_Fail;
			}
			else
			{
				int64 Size = Src->TotalSize();
				int64 Percent = 0, NewPercent = 0;
				uint8* Buffer = new uint8[COPYBLOCKSIZE];
				for( int64 Total = 0; Total < Size; Total += sizeof(Buffer) )
				{
					int64 Count = FMath::Min( Size - Total, (int64)sizeof(Buffer) );
					Src->Serialize( Buffer, Count );
					if( Src->IsError() )
					{
						Result = COPY_Fail;
						break;
					}
					Dest->Serialize( Buffer, Count );
					if( Dest->IsError() )
					{
						Result = COPY_Fail;
						break;
					}
					NewPercent = Total * 100 / Size;
					if( Progress && Percent != NewPercent && !Progress->Poll( ( float )NewPercent / 100.f ) )
					{
						Result = COPY_Canceled;
						break;
					}
					Percent = NewPercent;
				}
				delete [] Buffer;
				if( Result == COPY_OK && !Dest->Close() )
				{
					Result = COPY_Fail;
				}
				delete Dest;
				if( Result != COPY_OK )
				{
					Delete( *DestFile );
				}
			}
			if( Result == COPY_OK && !Src->Close() )
			{
				Result = COPY_Fail;
			}
			delete Src;
		}
		if( Progress && Result==COPY_OK && !Progress->Poll( 1.0 ) )
		{
			Result = COPY_Canceled;
		}
	}
	else
	{
		Result = COPY_Canceled;
	}

	return Result;
}

bool FFileManagerGeneric::Delete( const TCHAR* Filename, bool RequireExists, bool EvenReadOnly, bool Quiet )
{
	// Only allow writes to files that are not signed 
	// Except if the file is missing( that way corrupt ini files can be autogenerated by deleting them )
	if( FSHA1::GetFileSHAHash( Filename, NULL ) )
	{
		if (!Quiet)
		{
			UE_LOGF( LogFileManager, Log, "Can't delete signed game file: %ls",Filename );
		}
		return false;
	}
	bool bExists = GetLowLevel().FileExists( Filename );
	if( RequireExists && !bExists )
	{
		if (!Quiet)
		{
			UE_LOGF( LogFileManager, Warning, "Could not delete %ls because it doesn't exist.", Filename );
		}
		return false;
	}
	if( bExists )
	{
		if( EvenReadOnly )
		{
			GetLowLevel().SetReadOnly( Filename, false );
		}
		if( !GetLowLevel().DeleteFile( Filename ) )
		{
			if (!Quiet)
			{
				UE_LOGF( LogFileManager, Warning, "Error deleting file: %ls (Error Code %i)", Filename, FPlatformMisc::GetLastError() );
			}
			return false;
		}
	}
	return true;
}

bool FFileManagerGeneric::IsReadOnly( const TCHAR* Filename )
{
	return GetLowLevel().IsReadOnly( Filename );
}

bool FFileManagerGeneric::Move( const TCHAR* Dest, const TCHAR* Src, bool Replace, bool EvenIfReadOnly, bool Attributes, bool bDoNotRetryOrError )
{
	MakeDirectory( *FPaths::GetPath(Dest), true );
	// Retry on failure, unless the file wasn't there anyway.
	if( GetLowLevel().FileExists( Dest ) && Replace && !GetLowLevel().DeleteFile( Dest ) && !bDoNotRetryOrError )
	{
		// If the delete failed, throw a warning but retry before we throw an error
		UE_LOGF( LogFileManager, Warning, "DeleteFile was unable to delete '%ls', retrying in .5s...", Dest );

		// Wait just a little bit( i.e. a totally arbitrary amount )...
		FPlatformProcess::Sleep( 0.5f );

		// Try again
		if( !GetLowLevel().DeleteFile( Dest ) )
		{
			UE_LOGF( LogFileManager, Error, "Error deleting file '%ls'.", Dest );
			return false;
		}
		else
		{
			UE_LOGF( LogFileManager, Warning, "DeleteFile recovered during retry!" );
		}		
	}

	if( !GetLowLevel().MoveFile( Dest, Src ) )
	{
		if( bDoNotRetryOrError )
		{
			return false;
		}

		int32 RetryCount = 10;
		bool bSuccess = false;
		uint32 ErrorCode = FPlatformMisc::GetLastError();
		while (RetryCount--)
		{
			// If the move failed, throw a warning but retry before we throw an error
			UE_LOGF(LogFileManager, Warning, "MoveFile was unable to move '%ls' to '%ls' (Error Code %i), retrying in .5s...", Src, Dest, ErrorCode);

			// Wait just a little bit( i.e. a totally arbitrary amount )...
			FPlatformProcess::Sleep(0.5f);

			// Try again
			bSuccess = GetLowLevel().MoveFile(Dest, Src);
			if (bSuccess)
			{
				UE_LOGF(LogFileManager, Warning, "MoveFile recovered during retry!");
				break;
			}
			ErrorCode = FPlatformMisc::GetLastError();
		}
		if (!bSuccess)
		{
			UE_LOGF( LogFileManager, Error, "Error moving file '%ls' to '%ls'.", Src, Dest );
			return false;
		}
	}
	return true;
}

bool FFileManagerGeneric::FileExists( const TCHAR* Filename )
{
	return GetLowLevel().FileExists( Filename );
}

bool FFileManagerGeneric::DirectoryExists( const TCHAR* InDirectory )
{
	return GetLowLevel().DirectoryExists( InDirectory );
}

bool FFileManagerGeneric::MakeDirectory( const TCHAR* Path, bool Tree )
{
	if (Tree)
	{
		return GetLowLevel().CreateDirectoryTree(Path);
	}
	else
	{
		return GetLowLevel().CreateDirectory(Path);
	}
}

bool FFileManagerGeneric::DeleteDirectory( const TCHAR* Path, bool RequireExists, bool Tree )
{
	bool bSucceeded;
	if (Tree)
	{
		bSucceeded = GetLowLevel().DeleteDirectoryRecursively(Path);
	}
	else
	{
		bSucceeded = GetLowLevel().DeleteDirectory(Path);
	}

	if (!bSucceeded && !RequireExists)
	{
		uint32 ErrorFromDelete = FPlatformMisc::GetLastError();
		bSucceeded = !GetLowLevel().DirectoryExists(Path);
		if (!bSucceeded)
		{
			FPlatformMisc::SetLastError(ErrorFromDelete);
		}
	}
	return bSucceeded;
}

FFileStatData FFileManagerGeneric::GetStatData(const TCHAR* FilenameOrDirectory)
{
	return GetLowLevel().GetStatData(FilenameOrDirectory);
}

namespace FileManagerGenericImpl
{
	class FFileMatch : public IPlatformFile::FDirectoryVisitor
	{
	public:
		TArray<FString>& Result;
		FString WildCard;
		bool bFiles;
		bool bDirectories;
		bool bStoreFullPath;
		UE::FMutex ResultLock;

		FFileMatch(TArray<FString>& InResult, const FString& InWildCard, bool bInFiles, bool bInDirectories, bool bInStoreFullPath = false)
			: IPlatformFile::FDirectoryVisitor(EDirectoryVisitorFlags::ThreadSafe)
			, Result(InResult)
			, WildCard(InWildCard)
			, bFiles(bInFiles)
			, bDirectories(bInDirectories)
			, bStoreFullPath(bInStoreFullPath)
		{
		}

		virtual bool ShouldVisitLeafPathname(FStringView LeafFilename) override
		{
			return FString(LeafFilename).MatchesWildcard(WildCard);
		}

		virtual bool Visit(const TCHAR* FilenameOrDirectory, bool bIsDirectory)
		{
			if ((bIsDirectory && bDirectories) || (!bIsDirectory && bFiles))
			{
				FString Filename = FPaths::GetCleanFilename(FilenameOrDirectory);
				if (ensureMsgf(ShouldVisitLeafPathname(Filename),
					TEXT("PlatformFile.IterateDirectory needs to call ShouldVisitLeafFilename before calling Visit.")))
				{
					FString FullPath = bStoreFullPath ? FString(FilenameOrDirectory) : MoveTemp(Filename);
					UE::TScopeLock ScopeLock(ResultLock);
					Result.Add(MoveTemp(FullPath));
				}
			}
			return true;
		}
	};
}

void FFileManagerGeneric::FindFiles( TArray<FString>& Result, const TCHAR* InFilename, bool Files, bool Directories )
{
	FString Filename( InFilename );
	FPaths::NormalizeFilename( Filename );
	const FString CleanFilename = FPaths::GetCleanFilename(Filename);
	const bool bFindAllFiles = CleanFilename == TEXT("*") || CleanFilename == TEXT("*.*");
	FileManagerGenericImpl::FFileMatch FileMatch( Result, bFindAllFiles ? TEXT("*") : CleanFilename, Files, Directories );
	GetLowLevel().IterateDirectory( *FPaths::GetPath(Filename), FileMatch );
}

void FFileManagerGeneric::FindFiles(TArray<FString>& FoundFiles, const TCHAR* Directory, const TCHAR* FileExtension)
{
	if (!Directory)
	{
		return;
	}

	FString RootDir(Directory);
	FString ExtStr = (FileExtension != nullptr) ? FString(FileExtension) : "";

	// No Directory?
	if (RootDir.Len() < 1)
	{
		return;
	}

	FPaths::NormalizeDirectoryName(RootDir);

	// Don't modify the ExtStr if the user supplied the form "*.EXT" or "*" or "*.*" or "Name.*"
	if (!ExtStr.Contains(TEXT("*")))
	{
		if (ExtStr == "")
		{
			ExtStr = "*.*";
		}
		else
		{
			//Complete the supplied extension with * or *. to yield "*.EXT"
			ExtStr = (ExtStr.Left(1) == ".") ? "*" + ExtStr : "*." + ExtStr;
		}
	}

	// Create the full filter, which is "Directory/*.EXT".
	FString FinalPath = RootDir + "/" + ExtStr;
	FindFiles(FoundFiles, *FinalPath, true, false);
}

bool FFileManagerGeneric::IterateDirectory(const TCHAR* Directory, IPlatformFile::FDirectoryVisitor& Visitor)
{
	return GetLowLevel().IterateDirectory( Directory, Visitor );
}

bool FFileManagerGeneric::IterateDirectory(const TCHAR* Directory, IPlatformFile::FDirectoryVisitorFunc Visitor)
{
	return GetLowLevel().IterateDirectory( Directory, Visitor );
}

bool FFileManagerGeneric::IterateDirectoryRecursively(const TCHAR* Directory, IPlatformFile::FDirectoryVisitor& Visitor)
{
	return GetLowLevel().IterateDirectoryRecursively( Directory, Visitor );
}

bool FFileManagerGeneric::IterateDirectoryRecursively(const TCHAR* Directory, IPlatformFile::FDirectoryVisitorFunc Visitor)
{
	return GetLowLevel().IterateDirectoryRecursively( Directory, Visitor );
}

bool FFileManagerGeneric::IterateDirectoryStat(const TCHAR* Directory, IPlatformFile::FDirectoryStatVisitor& Visitor)
{
	return GetLowLevel().IterateDirectoryStat( Directory, Visitor );
}

bool FFileManagerGeneric::IterateDirectoryStat(const TCHAR* Directory, IPlatformFile::FDirectoryStatVisitorFunc Visitor)
{
	return GetLowLevel().IterateDirectoryStat( Directory, Visitor );
}

bool FFileManagerGeneric::IterateDirectoryStatRecursively(const TCHAR* Directory, IPlatformFile::FDirectoryStatVisitor& Visitor)
{
	return GetLowLevel().IterateDirectoryStatRecursively( Directory, Visitor );
}

bool FFileManagerGeneric::IterateDirectoryStatRecursively(const TCHAR* Directory, IPlatformFile::FDirectoryStatVisitorFunc Visitor)
{
	return GetLowLevel().IterateDirectoryStatRecursively( Directory, Visitor );
}

double FFileManagerGeneric::GetFileAgeSeconds( const TCHAR* Filename )
{
	// make sure it exists
	if (!GetLowLevel().FileExists(Filename))
	{
		return -1.0;
	}
	// get difference in time between now (UTC) and the filetime
	FTimespan Age = FDateTime::UtcNow() - GetTimeStamp(Filename);
	return Age.GetTotalSeconds();
}

FDateTime FFileManagerGeneric::GetTimeStamp(const TCHAR* Filename)
{
	// ask low level for timestamp
	return GetLowLevel().GetTimeStamp(Filename);
}

void FFileManagerGeneric::GetTimeStampPair(const TCHAR* PathA, const TCHAR* PathB, FDateTime& OutTimeStampA, FDateTime& OutTimeStampB)
{
	GetLowLevel().GetTimeStampPair(PathA, PathB, OutTimeStampA, OutTimeStampB);
}

bool FFileManagerGeneric::SetTimeStamp(const TCHAR* Filename, FDateTime DateTime)
{
	// if the file doesn't exist, fail
	if (!GetLowLevel().FileExists(Filename))
	{
		return false;
	}

	GetLowLevel().SetTimeStamp(Filename, DateTime);
	return true;
}

FDateTime FFileManagerGeneric::GetAccessTimeStamp( const TCHAR* Filename )
{
	// ask low level for timestamp
	return GetLowLevel().GetAccessTimeStamp(Filename);
}

FString FFileManagerGeneric::GetFilenameOnDisk(const TCHAR* Filename)
{
	return GetLowLevel().GetFilenameOnDisk(Filename);
}

bool FFileManagerGeneric::IsSymlink(const TCHAR* Filename)
{
	return GetLowLevel().IsSymlink(Filename) == ESymlinkResult::Symlink;
}

FString FFileManagerGeneric::DefaultConvertToRelativePath( const TCHAR* Filename )
{
	//default to the full absolute path of this file
	FString RelativePath( Filename );
	FPaths::NormalizeFilename(RelativePath);

	// See whether it is a relative path.
	FString RootDirectory( FPlatformMisc::RootDir() );
	FPaths::NormalizeFilename(RootDirectory);

	//the default relative directory it to the app root which is 3 directories up from the starting directory
	int32 NumberOfDirectoriesToGoUp = 3;

	//temp holder for current position of the slash
	int32 CurrentSlashPosition;

	//while we haven't run out of parent directories until we which a drive name
	while( ( CurrentSlashPosition = RootDirectory.Find( TEXT("/"), ESearchCase::CaseSensitive, ESearchDir::FromEnd ) ) != INDEX_NONE )
	{
		if( RelativePath.StartsWith( RootDirectory ) )
		{
			FString BinariesDir = FString(FPlatformProcess::BaseDir());
			FPaths::MakePathRelativeTo( RelativePath, *BinariesDir );
			break;
		}
		int32 PositionOfNextSlash = RootDirectory.Find( TEXT("/"), ESearchCase::CaseSensitive, ESearchDir::FromEnd, CurrentSlashPosition );
		//if there is another slash to find
		if( PositionOfNextSlash != INDEX_NONE )
		{
			//move up a directory and on an extra .. TEXT("/")
			// the +1 from "InStr" moves to include the "\" at the end of the directory name
			NumberOfDirectoriesToGoUp++;
			RootDirectory.LeftInline( PositionOfNextSlash + 1, EAllowShrinking::No);
		}
		else
		{
			RootDirectory.Empty();
		}
	}
	return RelativePath;
}

FString FFileManagerGeneric::ConvertToRelativePath( const TCHAR* Filename )
{
	return DefaultConvertToRelativePath(Filename);
}

FString FFileManagerGeneric::ConvertToAbsolutePathForExternalAppForRead( const TCHAR* AbsolutePath )
{
	return GetLowLevel().ConvertToAbsolutePathForExternalAppForRead( AbsolutePath );
}

FString FFileManagerGeneric::ConvertToAbsolutePathForExternalAppForWrite( const TCHAR* AbsolutePath )
{
	return GetLowLevel().ConvertToAbsolutePathForExternalAppForWrite( AbsolutePath );
}

void FFileManagerGeneric::FindFilesRecursive( TArray<FString>& FileNames, const TCHAR* StartDirectory, const TCHAR* Filename, bool Files, bool Directories, bool bClearFileNames)
{
	if( bClearFileNames )
	{
		FileNames.Empty();
	}
	FindFilesRecursiveInternal(FileNames, StartDirectory, Filename, Files, Directories);
}

void FFileManagerGeneric::FindFilesRecursiveInternal( TArray<FString>& FileNames, const TCHAR* StartDirectory, const TCHAR* InFilename, bool Files, bool Directories)
{
	FString Filename(InFilename);
	FPaths::NormalizeFilename(Filename);
	const FString CleanFilename = FPaths::GetCleanFilename(Filename);
	const bool bFindAllFiles = CleanFilename == TEXT("*") || CleanFilename == TEXT("*.*");
	const bool bStoreFullPath = true;
	FileManagerGenericImpl::FFileMatch FileMatch(FileNames, bFindAllFiles ? TEXT("*") : CleanFilename, Files, Directories, bStoreFullPath);
	GetLowLevel().IterateDirectoryRecursively(StartDirectory, FileMatch);
}

FArchiveFileReaderGeneric::FArchiveFileReaderGeneric( IFileHandle* InHandle, const TCHAR* InFilename, int64 InSize, uint32 InBufferSize, uint32 InFlags )
	: Filename( InFilename )
	, Size( InSize )
	, Pos( 0 )
	, BufferBase( 0 )
	, Handle( InHandle )
	, Flags( InFlags )
	, bFirstReadAfterSeek(false)
{
	BufferSize = FMath::Min(FMath::RoundUpToPowerOfTwo64((int64)InBufferSize), (uint64)Size);
	this->SetIsLoading(true);
	this->SetIsPersistent(true);
}


void FArchiveFileReaderGeneric::Seek( int64 InPos )
{
	checkf(InPos >= 0, TEXT("Attempted to seek to a negative location (%lld/%lld), file: %s. The file is most likely corrupt."), InPos, Size, *Filename);
	checkf(InPos <= Size, TEXT("Attempted to seek past the end of file (%lld/%lld), file: %s. The file is most likely corrupt."), InPos, Size, *Filename);

	int64 SeekPos;
	const bool bIsPosOutsideBufferWindow = (InPos < BufferBase) || (InPos >= (BufferBase + BufferArray.Num()));
	if (bIsPosOutsideBufferWindow)
	{
		SeekPos = InPos;
		bFirstReadAfterSeek = true;
	}
	else
	{
		SeekPos = BufferBase + BufferArray.Num();
		bFirstReadAfterSeek = false; // If we seek away, don't read anything, and then seek back to our buffer, treat the precaching heuristic as if we did not seek at all.
	}

	if (!SeekLowLevel(SeekPos))
	{
		uint32 LastError = FPlatformMisc::GetLastError();
		TCHAR ErrorBuffer[1024];
		SetError();
		UE_CLOGF( !IsSilent(), LogFileManager, Error, "SetFilePointer on %ls Failed %lld/%lld: %lld LastError==%u: %ls",
			*Filename, InPos, Size, Pos, LastError, FPlatformMisc::GetSystemErrorMessage(ErrorBuffer, 1024, LastError));
	}

	Pos = InPos;
}

FArchiveFileReaderGeneric::~FArchiveFileReaderGeneric()
{
	Close();
}

void FArchiveFileReaderGeneric::ReadLowLevel( uint8* Dest, int64 CountToRead, int64& OutBytesRead )
{
	if( Handle->Read( Dest, CountToRead ) )
	{
		OutBytesRead = CountToRead;
	}
	else
	{
		OutBytesRead = 0;
	}
}

bool FArchiveFileReaderGeneric::SeekLowLevel( int64 InPos )
{
	return Handle->Seek( InPos );
}

void FArchiveFileReaderGeneric::CloseLowLevel()
{
	Handle.Reset();
}

bool FArchiveFileReaderGeneric::Close()
{
	CloseLowLevel();
	return !IsError();
}

bool FArchiveFileReaderGeneric::Precache(int64 PrecacheOffset, int64 PrecacheSize)
{
	// Archives not based on async I/O should always return true, so we return true whether or not the precache was successful.
	// Returning false would imply that the caller should continue calling until it returns true.
	if (PrecacheSize < 0)
	{
		return true;
	}
	InternalPrecache(PrecacheOffset, PrecacheSize);
	bFirstReadAfterSeek = false; // If the caller is telling us to precache, then we should update the first read after seek heuristic to expect multiple reads
	return true;
}

bool FArchiveFileReaderGeneric::InternalPrecache( int64 PrecacheOffset, int64 PrecacheSize )
{
	// We defer allocation of BufferArray until we actually need it.  If the client doesn't do sequential reads
	// or explicitly request precaching via "FArchiveFileReaderGeneric::Precache", InternalPrecache might never
	// be called, and the buffer never actually used, a waste of 1 MB per archive file.
	if (BufferArray.Max() == 0)
	{
		BufferArray.Reserve(BufferSize);
	}

	// Only precache at current position and avoid work if precaching same offset twice.
	if (Pos != PrecacheOffset)
	{
		// We are refusing to precache, but return true if at least one byte after the requested PrecacheOffset is in our existing buffer.
		return BufferBase <= PrecacheOffset && PrecacheOffset < BufferBase + BufferArray.Num();
	}

	int64 BufferCount = FMath::Min(BufferSize, Size - Pos); // When we need to read more to satisfy the precache request, ignore the requested precachesize and set BufferCount to our buffersize.
	if (BufferCount <= 0)
	{
		return false;
	}
	bool bPosWithinBuffer = BufferBase <= Pos && Pos < BufferBase + BufferArray.Num();

	int64 ReadCount = BufferCount;
	int64 WriteOffset = 0;
	if (bPosWithinBuffer)
	{
		// At least one byte after the requested PrecacheOffset is in our existing buffer (since bPosWithinBuffer is true), so do nothing and return true
		return true;
	}
	else
	{
		// If we do not have an existing buffer, or Pos is outside it, the low-level read position is equal to Pos.
		// Read the next BufferCount bytes out of the low-level handle.
		BufferArray.SetNumUninitialized(BufferCount, EAllowShrinking::No);
	}
	BufferBase = Pos;

	check(ReadCount > 0); // If we don't need to read anything we should have early exited above
	int64 Count = 0;
	ReadLowLevel( BufferArray.GetData() + WriteOffset, ReadCount, Count );

	if (Count!=ReadCount)
	{
		uint32 LastError = FPlatformMisc::GetLastError();
		TCHAR ErrorBuffer[1024];
		UE_CLOGF( !IsSilent(), LogFileManager, Warning, "ReadFile failed: Count=%lld ReadCount=%lld LastError=%u: %ls",
			Count, ReadCount, LastError, FPlatformMisc::GetSystemErrorMessage( ErrorBuffer, 1024, LastError) );
		BufferCount = WriteOffset + Count;
		BufferArray.SetNumUninitialized(BufferCount);
		// The read failed, but we do not SetError or return false just because a precache read fails.
		// Return true if at least one byte after the requested PrecacheOffset was read or is in our existing buffer.
		return BufferCount > 0;
	}
	return true;
}

void FArchiveFileReaderGeneric::Serialize( void* V, int64 Length )
{
	if (Pos + Length > Size)
	{
		SetError();
		UE_CLOG( !IsSilent(), LogFileManager, Error, TEXT("Requested read of %" INT64_FMT " bytes when %" INT64_FMT " bytes remain (file=%s, size=%" INT64_FMT ")"), Length, Size-Pos, *Filename, Size);
		return;
	}

	const bool bIsOutsideBufferWindow = (Pos < BufferBase) || (Pos >= (BufferBase + BufferArray.Num()));
	bool bReadUncached = false;
	if (bIsOutsideBufferWindow)
	{
		// This is likely due to a seek that has happened in the meantime.
		BufferBase = Pos;
		BufferArray.Reset();
		// We use a heuristic to decide when we should skip preloading the buffer. It is common in Unreal packages to seek to a spot, do a single small read, and then seek away to the next spot.
		// We should therefore not waste time populating the buffer for the seek point after a seek.
		// If we continue reading from that point, then the heuristic no longer applies and we should populate the buffer as normal.
		bReadUncached = bFirstReadAfterSeek;
	}
	bFirstReadAfterSeek = false;

	while( Length>0 )
	{
		int64 Copy = FMath::Min( Length, BufferBase+BufferArray.Num()-Pos );
		if( Copy<=0 )
		{
			if( Length >= BufferSize || bReadUncached )
			{
				int64 Count=0;
				{
					ReadLowLevel(( uint8* )V, Length, Count );
				}
				if( Count!=Length )
				{
					uint32 LastError = FPlatformMisc::GetLastError();
					TCHAR ErrorBuffer[1024];
					SetError();
					UE_CLOGF( !IsSilent(), LogFileManager, Warning, "ReadFile failed: Count=%lld Length=%lld Error=%u: %ls for file %ls", 
						Count, Length, LastError, FPlatformMisc::GetSystemErrorMessage( ErrorBuffer, 1024, LastError),
						*Filename );
				}
				Pos += Length;
				return;
			}
			if (!InternalPrecache(Pos, MAX_int32))
			{
				SetError();
				UE_CLOGF( !IsSilent(), LogFileManager, Warning, "ReadFile failed during precaching for file %ls",*Filename );
				return;
			}
			Copy = FMath::Min( Length, BufferBase+BufferArray.Num()-Pos );
			if( Copy<=0 )
			{
				SetError();
				UE_CLOGF( !IsSilent(), LogFileManager, Error, "ReadFile beyond EOF %lld+%lld/%lld for file %ls", 
					Pos, Length, Size, *Filename );
			}
			if( IsError() )
			{
				return;
			}
		}
		FMemory::Memcpy( V, BufferArray.GetData()+Pos-BufferBase, Copy );
		Pos       += Copy;
		Length    -= Copy;
		V          =( uint8* )V + Copy;
	}
}

void FArchiveFileReaderGeneric::FlushCache()
{
	BufferArray.Empty();
	// After clearing BufferArray, we need to set BufferBase and LowLevel to Pos. They may have been up to BufferSize away.
	BufferBase = Pos;

	if (Handle.IsValid())
	{
		Handle->ShrinkBuffers();
		// After clearing BufferArray, we need to set BufferBase and LowLevel to Pos. They may have been up to BufferSize away.
		SeekLowLevel(Pos);
	}
}

FArchiveFileWriterGeneric::FArchiveFileWriterGeneric( IFileHandle* InHandle, const TCHAR* InFilename, int64 InPos, uint32 InBufferSize, uint32 InFlags )
	: Filename( InFilename )
	, Flags( InFlags )
	, Pos( InPos )
	, Handle( InHandle )
	, bLoggingError( false )
{
	BufferSize = FMath::RoundUpToPowerOfTwo64((int64)InBufferSize);
	BufferArray.Reserve(BufferSize);
	this->SetIsSaving(true);
	this->SetIsPersistent(true);
}

FArchiveFileWriterGeneric::~FArchiveFileWriterGeneric()
{
	Close();
}

bool FArchiveFileWriterGeneric::CloseLowLevel()
{
	Handle.Reset();
	return true;
}

bool FArchiveFileWriterGeneric::SeekLowLevel( int64 InPos )
{
	return Handle->Seek( InPos );
}

int64 FArchiveFileWriterGeneric::TotalSize()
{
	return FMath::Max(Pos, Handle->Size());
}

bool FArchiveFileWriterGeneric::WriteLowLevel( const uint8* Src, int64 CountToWrite )
{
	return Handle->Write( Src, CountToWrite );
}

void FArchiveFileWriterGeneric::Seek(int64 InPos)
{
	if (InPos != Pos)
	{
		FlushBuffer();
		if (!SeekLowLevel(InPos))
		{
			uint32 LastError = FPlatformMisc::GetLastError();
			SetError();
			LogWriteError(TEXT("Error seeking file"), LastError);
		}
		Pos = InPos;
	}
}

bool FArchiveFileWriterGeneric::Close()
{
	FlushBuffer();
	if( !CloseLowLevel() )
	{
		uint32 LastError = FPlatformMisc::GetLastError();
		SetError();
		LogWriteError(TEXT("Error closing file"), LastError);
	}
	return !IsError();
}

void FArchiveFileWriterGeneric::Serialize( void* V, int64 Length )
{
	Pos += Length;
	if ( Length >= BufferSize )
	{
		FlushBuffer();
		if( !WriteLowLevel( (uint8*)V, Length ) )
		{
			uint32 LastError = FPlatformMisc::GetLastError();
			SetError();
			LogWriteError(TEXT("Error writing to file"), LastError);
		}
	}
	else
	{
		int64 Copy;
		while( Length >( Copy=BufferSize-BufferArray.Num() ) )
		{
			BufferArray.Append((uint8*)V, Copy);
			Length      -= Copy;
			V            =( uint8* )V + Copy;
			FlushBuffer();
		}
		if( Length )
		{
			BufferArray.Append((uint8*)V, Length);
		}
	}
}

void FArchiveFileWriterGeneric::Flush()
{
	if (FlushBuffer() && Handle)
	{
		Handle->Flush();
	}
}

bool FArchiveFileWriterGeneric::FlushBuffer()
{
	bool bDidWriteData = false;
	if (int64 BufferNum = BufferArray.Num())
	{
		bDidWriteData = WriteLowLevel(BufferArray.GetData(), BufferNum);
		if (!bDidWriteData)
		{
			uint32 LastError = FPlatformMisc::GetLastError();
			SetError();
			LogWriteError(TEXT("Error flushing file"), LastError);
		}
		BufferArray.Reset();
	}
	return bDidWriteData;
}

void FArchiveFileWriterGeneric::LogWriteError(const TCHAR* Message, uint32 LastError)
{
	// Prevent re-entry if logging causes another log error leading to a stack overflow
	if (!bLoggingError && !IsSilent())
	{
		bLoggingError = true;
		TCHAR ErrorBuffer[1024];
		UE_LOGF(LogFileManager, Error, "%ls: %ls (%u: %ls)", Message, *Filename,
			LastError, FPlatformMisc::GetSystemErrorMessage(ErrorBuffer, 1024, LastError));
		bLoggingError = false;
	}
}
//---


IFileManager& IFileManager::Get()
{
	UE::AccessDetection::ReportAccess(UE::AccessDetection::EType::File);
	static FFileManagerGeneric Singleton;
	return Singleton;
}
