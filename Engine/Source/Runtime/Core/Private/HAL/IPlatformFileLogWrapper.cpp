// Copyright Epic Games, Inc. All Rights Reserved.

#include "HAL/IPlatformFileLogWrapper.h"
#include "Misc/CoreMisc.h"
#include "Templates/UniquePtr.h"
#include "Async/MappedFileHandle.h"

bool bSuppressFileLog = false;
DEFINE_LOG_CATEGORY(LogPlatformFile);


#if !UE_BUILD_SHIPPING
class FFileLogExec : private FSelfRegisteringExec
{
	FLoggedPlatformFile& PlatformFile;

public:

	FFileLogExec(FLoggedPlatformFile& InPlatformFile)
		: PlatformFile(InPlatformFile)
	{}

	/** Console commands **/
	virtual bool Exec_Dev(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override
	{
		if (FParse::Command(&Cmd, TEXT("LogFileDump")))
		{
			PlatformFile.HandleDumpCommand(Cmd, Ar);
			return true;
		}
		return false;
	}
};
static TUniquePtr<FFileLogExec> GFileLogExec;

#endif // !UE_BUILD_SHIPPING


bool FLoggedPlatformFile::ShouldBeUsed(IPlatformFile* Inner, const TCHAR* CmdLine) const
{
	return FParse::Param(CmdLine, TEXT("FileLog"));
}

bool FLoggedPlatformFile::Initialize(IPlatformFile* Inner, const TCHAR* CommandLineParam)
{
	// Inner is required.
	check(Inner != nullptr);
	LowerLevel = Inner;

#if !UE_BUILD_SHIPPING
	GFileLogExec = MakeUnique<FFileLogExec>(*this);
#endif

	return !!LowerLevel;
}

#if !UE_BUILD_SHIPPING
void FLoggedPlatformFile::HandleDumpCommand(const TCHAR* Cmd, FOutputDevice& Ar)
{
	FScopeLock LogFileLock(&LogFileCritical);
	bSuppressFileLog = true;
	Ar.Logf(TEXT("Open file handles: %d"), OpenHandles.Num());
	for (auto& HandlePair : OpenHandles)
	{
		Ar.Logf(TEXT("%s: %d"), *HandlePair.Key, HandlePair.Value);
	}	
	bSuppressFileLog = false;
}
#endif

FLoggedFileHandle::FLoggedFileHandle(IFileHandle* InFileHandle, const TCHAR* InFilename, FLoggedPlatformFile& InOwner)
: FileHandle(InFileHandle)
, Filename(InFilename)
#if !UE_BUILD_SHIPPING
, PlatformFile(InOwner)
#endif
{
#if !UE_BUILD_SHIPPING
	PlatformFile.OnHandleOpen(Filename);
#endif
}

FLoggedFileHandle::~FLoggedFileHandle()
{
#if !UE_BUILD_SHIPPING
	PlatformFile.OnHandleClosed(Filename);
#endif
	FILE_LOG(LogPlatformFile, Log, TEXT("Close %s"), *Filename);
}

FOpenMappedResult FLoggedPlatformFile::OpenMappedEx(const TCHAR* Filename, EOpenReadFlags OpenOptions, int64 MaximumSize)
{
	return LowerLevel->OpenMappedEx(Filename, OpenOptions, MaximumSize);
}

FOpenMappedResult FLoggedPlatformFile::OpenMappedEx2(const TCHAR* Filename, EOpenMappedFlags OpenOptions, int64 MaximumSize)
{
	return LowerLevel->OpenMappedEx2(Filename, OpenOptions, MaximumSize);
}
