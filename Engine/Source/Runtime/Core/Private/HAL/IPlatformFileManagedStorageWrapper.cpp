// Copyright Epic Games, Inc. All Rights Reserved.

#include "HAL/IPlatformFileManagedStorageWrapper.h"
#include "HAL/IConsoleManager.h"
#include "HAL/FileManager.h"
#include "Algo/Accumulate.h"

DEFINE_LOG_CATEGORY(LogPlatformFileManagedStorage);

#if !UE_BUILD_SHIPPING
static FAutoConsoleCommand PersistentStorageCategoryStatsCommand
(
	TEXT("PersistentStorageCategoryStats"),
	TEXT("Get the stat of each persistent storage stats\n"),
	FConsoleCommandDelegate::CreateStatic([]()
{
	for (auto& CategoryStat : FPersistentStorageManager::Get().GenerateCategoryStats())
	{
		UE_LOGF(LogPlatformFileManagedStorage, Display, "%ls", *CategoryStat.Value.Print());
	}
})
);

static FAutoConsoleCommandWithOutputDevice DumpPersistentStorage(
	TEXT("DumpPersistentStorage"),
	TEXT("Dumps PersistentStorage"),
	FConsoleCommandWithOutputDeviceDelegate::CreateStatic([](FOutputDevice& Output)
		{
			if (!FPaths::HasProjectPersistentDownloadDir())
			{
				Output.Log(TEXT("No persistent download dir configured."));
				return;
			}

			IFileManager::Get().IterateDirectoryStatRecursively(*FPaths::ProjectPersistentDownloadDir(),
				[&Output](const TCHAR* FileOrDir, const FFileStatData& StatData)
				{
					if (StatData.bIsValid && !StatData.bIsDirectory)
					{
						Output.Logf(TEXT("%" INT64_FMT "\t %s"), StatData.FileSize, FileOrDir);
					}
					return true;
				}
			);
		}
	)
);

static FAutoConsoleCommandWithWorldArgsAndOutputDevice VerifyPersistentStorageCategory(
	TEXT("VerifyPersistentStorageCategory"),
	TEXT("VerifyPersistentStorageCategory <Category>"),
	FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateLambda([](const TArray<FString>& Args, UWorld*, FOutputDevice& Output)
		{
			if (Args.Num() < 1)
			{
				Output.Log(ELogVerbosity::Error, TEXT("A category must be specified"));
				return;
			}

			TOptional<FPersistentStorageCategory::CategoryStat> MaybeStat = FPersistentStorageManager::Get().GetCategoryStat(Args[0]);
			if (!MaybeStat)
			{
				Output.Logf(ELogVerbosity::Error, TEXT("Category %s not found"), *Args[0]);
				return;
			}

			Output.Logf(TEXT("%s"), *MaybeStat->Print());

			Output.Log(TEXT("Note: results may not be accurate if category is modified during verification."));

			bool bVerified = true;
			int64 TotalTrackedFileSize = Algo::TransformAccumulate(MaybeStat->FileSizes, &TPair<FString, int64>::Value, int64(0));
			if (TotalTrackedFileSize != MaybeStat->UsedSize)
			{
				Output.Logf(ELogVerbosity::Error, TEXT("TotalTrackedFileSize %" INT64_FMT " does not match UsedSize %" INT64_FMT), TotalTrackedFileSize, MaybeStat->UsedSize);
				bVerified = false;
			}

			int64 TotalSizeOnDisk = 0;
			for (const FString& RootDir : FPersistentStorageManager::Get().GetRootDirectories())
			{
				IFileManager::Get().IterateDirectoryStatRecursively(*RootDir,
					[&MaybeStat, &TotalSizeOnDisk](const TCHAR* FileOrDir, const FFileStatData& StatData)
					{
						if (StatData.bIsValid && !StatData.bIsDirectory)
						{
							for (const FString& Dir : MaybeStat->Directories)
							{
								if (ManagedStorageInternal::IsUnderDirectory(FileOrDir, Dir))
								{
									TotalSizeOnDisk += StatData.FileSize;
									break;
								}
							}
						}
						return true;
					});
			}

			if (TotalSizeOnDisk != MaybeStat->UsedSize)
			{
				Output.Logf(ELogVerbosity::Error, TEXT("TotalSizeOnDisk %" INT64_FMT " does not match UsedSize %" INT64_FMT), TotalSizeOnDisk, MaybeStat->UsedSize);
				bVerified = false;
			}

			if (bVerified)
			{
				Output.Log(TEXT("Category Verified!"));
			}
		}
	)
);

static FAutoConsoleCommand CreateDummyFileInPersistentStorageCommand(
	TEXT("CreateDummyFileInPersistentStorage"),
	TEXT("Create a dummy file with specified size in specified persistent storage folder"),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
{
	if (Args.Num() < 2)
	{
		UE_LOGF(LogPlatformFileManagedStorage, Error, "Not enough parameters to run console command CreateDummyFileInPersistentStorage");
		return;
	}

	// Args[0]: FilePath
	// Args[1]: Size
	const FString& DummyFilePath = Args[0];
	if (!FPaths::IsUnderDirectory(DummyFilePath, TEXT("/download0")))
	{
		UE_LOGF(LogPlatformFileManagedStorage, Error, "Failed to write dummy file %ls.  File path is not under /download0", *DummyFilePath);
		return;
	}

	int32 FileSize;;
	LexFromString(FileSize, *Args[1]);
	int32 BufferSize = 16 * 1024;
	TArray<uint8> DummyBuffer;
	DummyBuffer.SetNum(BufferSize);

	TUniquePtr<FArchive> Ar = TUniquePtr<FArchive>(IFileManager::Get().CreateFileWriter(*DummyFilePath, 0));
	if (!Ar)
	{
		UE_LOGF(LogPlatformFileManagedStorage, Error, "Failed to write dummy file %ls.", *DummyFilePath);
		return;
	}

	int32 RemainingBytesToWrite = FileSize;
	while (RemainingBytesToWrite > 0)
	{
		int32 SizeToWrite = FMath::Min(RemainingBytesToWrite, BufferSize);
		Ar->Serialize(const_cast<uint8*>(DummyBuffer.GetData()), SizeToWrite);
		RemainingBytesToWrite -= SizeToWrite;
	}

	if(!Ar->Close())
	{
		UE_LOGF(LogPlatformFileManagedStorage, Error, "There was an error writing to file %ls.", *DummyFilePath);
	}
}),
ECVF_Default);
#endif // !UE_BUILD_SHIPPING

bool ManagedStorageInternal::IsUnderDirectory(const FString& Path, const FString& Directory)
{
	// ConvertRelativePathToFull() should have been called on any path being passed to this function, but on many platforms BaseDir is TEXT("")
	// 	   which makes all paths stay relative to it anyway so its possible we may have relative paths at this point.
	//check(!FPaths::IsRelative(Path));
	//check(!FPaths::IsRelative(Directory));

	int32 DirectoryLen = Directory.Len();
	if (Directory.EndsWith(TEXT("/")))
	{
		DirectoryLen -= 1;
	}

	int Compare = FCString::Strnicmp(*Path, *Directory, DirectoryLen);

	return Compare == 0 && (Path.Len() == Directory.Len() || Path[DirectoryLen] == TEXT('/'));
}

void FManagedStorageScopeFileLock::Unlock()
{
	if (pFileCS)
	{
		pFileCS->Unlock();
		FPersistentStorageManager::Get().Categories.GetCategories()[ManagedFile.Category].GetLockRegistry().ReleaseLock(ManagedFile.FullFilename);
		pFileCS = nullptr;
	}
}

void FManagedStorageScopeFileLock::Lock()
{
	if (ManagedFile)
	{
		pFileCS = FPersistentStorageManager::Get().Categories.GetCategories()[ManagedFile.Category].GetLockRegistry().GetLock(ManagedFile.FullFilename);
		pFileCS->Lock();
	}
}

FPersistentStorageManager& FPersistentStorageManager::Get()
{
	check(IsReady());
	static FPersistentStorageManager Singleton;
	return Singleton;
}

FPersistentStorageManager::FPersistentStorageManager()
	: bInitialized(false)
	, Categories(InitCategories())
{
	RootDirectories.Init(Categories.GetCategories());
}

FPersistentStorageManager::FCategoryInfo FPersistentStorageManager::InitCategories()
{
	check(GConfig);

	TArray<FPersistentStorageCategory> Categories;
	int32 DefaultCategory = -1;

	TMap<FString, FString> CustomDirectoryReplace;
	CustomDirectoryReplace.Add(TEXT("[persistent]"), FPaths::ConvertRelativePathToFull(FPaths::ProjectPersistentDownloadDir()));
	CustomDirectoryReplace.Add(TEXT("[saved]"), FPaths::ConvertRelativePathToFull(FPaths::ProjectSavedDir()));

	for (TPair<FString, FString>& Pair : CustomDirectoryReplace)
	{
		if (!Pair.Value.EndsWith(TEXT("/")))
		{
			Pair.Value += TEXT("/");
		}
	}

	// Takes on the pattern
	// (Name="CategoryName",QuotaMB=100,Directories=("Dir1","Dir2","Dir3"))
	TArray<FString> CategoryConfigs;
	GConfig->GetArray(TEXT("PersistentStorageManager"), TEXT("Categories"), CategoryConfigs, GEngineIni);

	Categories.Empty(CategoryConfigs.Num());
	for (const FString& Category : CategoryConfigs)
	{
		FString TrimmedCategory = Category;
		TrimmedCategory.TrimStartAndEndInline();
		if (TrimmedCategory.Left(1) == TEXT("("))
		{
			TrimmedCategory.RightChopInline(1, EAllowShrinking::No);
		}
		if (TrimmedCategory.Right(1) == TEXT(")"))
		{
			TrimmedCategory.LeftChopInline(1, EAllowShrinking::No);
		}

		// Find all custom chunks and parse
		const TCHAR* PropertyName = TEXT("Name=");
		const TCHAR* PropertyQuotaMB = TEXT("QuotaMB=");
		const TCHAR* PropertyDirectories = TEXT("Directories=");
		const TCHAR* PropertyOptionalQuotaMB = TEXT("OptionalQuotaMB=");
		FString CategoryName;
		int64 QuotaInMB;
		FString DirectoryNames;

		if (FParse::Value(*TrimmedCategory, PropertyName, CategoryName) &&
			FParse::Value(*TrimmedCategory, PropertyQuotaMB, QuotaInMB) &&
			FParse::Value(*TrimmedCategory, PropertyDirectories, DirectoryNames, false))
		{
			if (CategoryName.Len() == 0)
			{
				UE_LOGF(LogPlatformFileManagedStorage, Error, "Found empty category in [PersistentStorageManager]:Categories");
				continue;
			}

			int64 OptionalQuotaMB;
			if (FParse::Value(*TrimmedCategory, PropertyOptionalQuotaMB, OptionalQuotaMB) == false)
			{
				OptionalQuotaMB = 0;
			}

			CategoryName.ReplaceInline(TEXT("\""), TEXT(""));

			// Split Directories
			if (DirectoryNames.Left(1) == TEXT("("))
			{
				DirectoryNames.RightChopInline(1, EAllowShrinking::No);
			}
			if (DirectoryNames.Right(1) == TEXT(")"))
			{
				DirectoryNames.LeftChopInline(1, EAllowShrinking::No);
			}

			TArray<FString> Directories;
			DirectoryNames.ParseIntoArray(Directories, TEXT(","));
			for (FString& DirectoryName : Directories)
			{
				DirectoryName.ReplaceInline(TEXT("\""), TEXT(""));

				bool bUsedCustomDirectory = false;
				for (const TPair<FString, FString>& DirectoryMapping : CustomDirectoryReplace)
				{
					if (DirectoryName.ReplaceInline(*DirectoryMapping.Key, *DirectoryMapping.Value) > 0)
					{
						bUsedCustomDirectory = true;
					}
				}

				if (bUsedCustomDirectory)
				{
					FPaths::NormalizeDirectoryName(DirectoryName);
				}
				else
				{
					// ConvertRelativePathToFull also normalizes
					DirectoryName = FPaths::ConvertRelativePathToFull(FPaths::ProjectPersistentDownloadDir() / DirectoryName);
				}
			}

			// see if there are any commandline config overrides

			int64 QuotaInMBOverride = -1;
			FString CommandLineOptionName = FString::Printf(TEXT("persistentstoragequota%s="), *CategoryName);
			if (FParse::Value(FCommandLine::Get(), *CommandLineOptionName, QuotaInMBOverride))
			{
				QuotaInMB = QuotaInMBOverride;
			}

			int64 Quota = (QuotaInMB >= 0) ? QuotaInMB * 1024 * 1024 : -1;	// Quota being negative means infinite quota
			int64 OptionalQuota = (OptionalQuotaMB >= 0) ? OptionalQuotaMB * 1024 * 1024 : 0;
			Categories.Emplace(MoveTemp(CategoryName), MoveTemp(Directories), Quota, OptionalQuota);
		}
	}

	FString DefaultCategoryName;
	GConfig->GetString(TEXT("PersistentStorageManager"), TEXT("DefaultCategoryName"), DefaultCategoryName, GEngineIni);
	DefaultCategory = Algo::IndexOfBy(Categories, DefaultCategoryName, [](const FPersistentStorageCategory& Cat) { return Cat.GetCategoryName(); });
	if (DefaultCategory == INDEX_NONE)
	{
		UE_LOGF(LogPlatformFileManagedStorage, Error, "Default category %ls doesn't exist", *DefaultCategoryName);
	}

	return FCategoryInfo(MoveTemp(Categories), DefaultCategory);
}

void FPersistentStorageManager::FRootDirInfo::Init(TArrayView<const FPersistentStorageCategory> InCategories)
{
	FString PersistentDownloadDir = FPaths::ConvertRelativePathToFull(FPaths::ProjectPersistentDownloadDir());

	for (const FPersistentStorageCategory& Category : InCategories)
	{
		for (const FString& Dir : Category.GetDirectories())
		{
			if (!ManagedStorageInternal::IsUnderDirectory(Dir, PersistentDownloadDir))
			{
				RootDirectories.Add(Dir);
			}
		}
	}

	RootDirectories.Add(MoveTemp(PersistentDownloadDir));
}

FPersistentStorageManager::FCategoryInfo::FCategoryInfo(TArray<FPersistentStorageCategory>&& InCategories, int32 InDefaultCategory) 
	: Categories(MoveTemp(InCategories))
	, DefaultCategory(InDefaultCategory)
{
}

