// Copyright Epic Games, Inc. All Rights Reserved.

#include "HAL/IPlatformFileOpenLogWrapper.h"

#if !UE_BUILD_SHIPPING

#include "Async/MappedFileHandle.h"

DEFINE_LOG_CATEGORY_STATIC(LogFileOpenOrder, Log, All);

static FPlatformFileOpenLog* GPlatformFileOpenLog = nullptr;

IAsyncReadRequest* FLoggingAsyncReadFileHandle::ReadRequest(int64 Offset, int64 BytesToRead, EAsyncIOPriorityAndFlags PriorityAndFlags, FAsyncFileCallBack* CompleteCallback, uint8* UserSuppliedMemory)
{
	if ((PriorityAndFlags & AIOP_FLAG_PRECACHE) == 0)
	{
		Owner->AddToOpenLog(*Filename);
	}
	return ActualRequest->ReadRequest(Offset, BytesToRead, PriorityAndFlags, CompleteCallback, UserSuppliedMemory);
}

bool FPlatformFileOpenLog::Initialize(IPlatformFile* Inner, const TCHAR* CommandLineParam)
{
	LowerLevel = Inner;
	FString LogFileDirectory;
	FString LogFilePath;
	FString PlatformStr;
	FString	OutputDirectoryBase;

	if (!FParse::Value(CommandLineParam, TEXT("FOOBASEDIR="), OutputDirectoryBase))
	{
		if (FParse::Param(CommandLineParam, TEXT("FOOUSESAVEDDIR")))
		{
			OutputDirectoryBase = FPaths::ProjectSavedDir();
		}
		else
		{
			OutputDirectoryBase = FPlatformMisc::ProjectDir();
		}
	}

	if (FParse::Value(CommandLineParam, TEXT("TARGETPLATFORM="), PlatformStr))
	{
		TArray<FString> PlatformNames;
		if (!(PlatformStr == TEXT("None") || PlatformStr == TEXT("All")))
		{
			PlatformStr.ParseIntoArray(PlatformNames, TEXT("+"), true);
		}

		for (int32 Platform = 0; Platform < PlatformNames.Num(); ++Platform)
		{
			if (FDataDrivenPlatformInfoRegistry::GetPlatformInfo(PlatformNames[Platform]).bIsConfidential)
			{
				LogFileDirectory = FPaths::Combine(OutputDirectoryBase, TEXT("Platforms"), *PlatformNames[Platform], TEXT("Build"), TEXT("FileOpenOrder"));
			}
			else
			{
				LogFileDirectory = FPaths::Combine(OutputDirectoryBase, TEXT("Build"), *PlatformNames[Platform], TEXT("FileOpenOrder"));
			}
#if WITH_EDITOR
			LogFilePath = FPaths::Combine(*LogFileDirectory, TEXT("EditorOpenOrder.log"));
#else 
			FString OrderSuffix;
			FParse::Value(CommandLineParam, TEXT("FooSuffix="), OrderSuffix);

			FString GameOpenOrderFileName = FString::Printf(TEXT("GameOpenOrder%s.log"), *OrderSuffix);
			LogFilePath = FPaths::Combine(*LogFileDirectory, *GameOpenOrderFileName);
#endif
			Inner->CreateDirectoryTree(*LogFileDirectory);
			auto* FileHandle = Inner->OpenWrite(*LogFilePath, false, false);
			if (FileHandle)
			{
				LogOutput.Add(FileHandle);
			}
			UE_LOGF(LogFileOpenOrder, Log, "Initialized file open order log : %ls", *LogFilePath);
		}
	}
	else
	{
		if (FDataDrivenPlatformInfoRegistry::GetPlatformInfo(FPlatformProperties::IniPlatformName()).bIsConfidential)
		{
			LogFileDirectory = FPaths::Combine(OutputDirectoryBase, TEXT("Platforms"), StringCast<TCHAR>(FPlatformProperties::PlatformName()).Get(), TEXT("Build"), TEXT("FileOpenOrder"));
		}
		else
		{
			LogFileDirectory = FPaths::Combine(OutputDirectoryBase, TEXT("Build"), StringCast<TCHAR>(FPlatformProperties::PlatformName()).Get(), TEXT("FileOpenOrder"));
		}
#if WITH_EDITOR
		LogFilePath = FPaths::Combine(*LogFileDirectory, TEXT("EditorOpenOrder.log"));
#else 
		LogFilePath = FPaths::Combine(*LogFileDirectory, TEXT("GameOpenOrder.log"));
#endif
		Inner->CreateDirectoryTree(*LogFileDirectory);
		auto* FileHandle = Inner->OpenWrite(*LogFilePath, false, false);
		if (FileHandle)
		{
			LogOutput.Add(FileHandle);
		}
		UE_LOGF(LogFileOpenOrder, Log, "Initialized file open order log : %ls", *LogFilePath);
	}

	// Log duplicates if requested. This can be used for debugging/analysis purposes to log file access order but shouldn't be used to generate a pak/iostore ordering
	bLogDuplicates = FParse::Param(CommandLineParam, TEXT("FooLogDuplicates"));

	GPlatformFileOpenLog = this;
	return true;
}

FOpenMappedResult FPlatformFileOpenLog::OpenMappedEx(const TCHAR* Filename, EOpenReadFlags OpenOptions, int64 MaximumSize)
{
	return LowerLevel->OpenMappedEx(Filename, OpenOptions, MaximumSize);
}

FOpenMappedResult FPlatformFileOpenLog::OpenMappedEx2(const TCHAR* Filename, EOpenMappedFlags OpenOptions, int64 MaximumSize)
{
	return LowerLevel->OpenMappedEx2(Filename, OpenOptions, MaximumSize);
}

void FPlatformFileOpenLog::AddLabelInternal(const TCHAR* Fmt, ...)
{
	if (GPlatformFileOpenLog)
	{
		TCHAR Buffer[256];
		GET_TYPED_VARARGS(TCHAR, Buffer, UE_ARRAY_COUNT(Buffer), UE_ARRAY_COUNT(Buffer) - 1, Fmt, Fmt);
		Buffer[255] = '\0';
		GPlatformFileOpenLog->AddLabelToOpenLog(Buffer);

		UE_LOGF(LogFileOpenOrder, Log, "Label: %ls", Buffer);
	}
}

#endif // !UE_BUILD_SHIPPING
