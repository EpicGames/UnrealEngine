// Copyright Epic Games, Inc. All Rights Reserved.

#include "HAL/LLM/ActiveTags.h"

#if ENABLE_LOW_LEVEL_MEM_TRACKER

namespace UE::LLMPrivate
{

FActiveTags::FActiveTags()
{
	const int32 GlobalNum = GetGlobalNum();
	for (int32 Index = 0; Index < GlobalNum; ++Index)
	{
		Values[Index] = nullptr;
	}
}

FActiveTags::FActiveTags(const FActiveTags& Other)
{
	*this = Other;
}

FActiveTags::FActiveTags(FActiveTags&& Other)
{
	*this = (FActiveTags&&)Other;
}

FActiveTags& FActiveTags::operator=(const FActiveTags& Other)
{
	const int32 GlobalNum = GetGlobalNum();
	for (int32 Index = 0; Index < GlobalNum; ++Index)
	{
		Values[Index] = Other.Values[Index];
	}
	return *this;
}

FActiveTags& FActiveTags::operator=(FActiveTags&& Other)
{
	const int32 GlobalNum = GetGlobalNum();
	for (int32 Index = 0; Index < GlobalNum; ++Index)
	{
		Values[Index] = Other.Values[Index];
	}
	return *this;
}

const FTagData*& FActiveTags::operator[](int32 Index)
{
	LLMCheck(0 <= Index);
	LLMCheck(Index < GetGlobalNum());
	return Values[Index];
}

const FTagData* const& FActiveTags::operator[](int32 Index) const
{
	LLMCheck(0 <= Index);
	LLMCheck(Index < GetGlobalNum());
	return Values[Index];
}

int32 FActiveTags::Num() const
{
	return GetGlobalNum();
}

bool FActiveTags::operator==(const FActiveTags& Other) const
{
	const int32 GlobalNum = GetGlobalNum();
	for (int32 Index = 0; Index < GlobalNum; ++Index)
	{
		if (Values[Index] != Other.Values[Index])
		{
			return false;
		}
	}
	return true;
}

const FTagData* FActiveTags::GetSystemsTagData() const
{
	LLMCheck(GetGlobalNum() > 0);
	return Values[0];
}

void FActiveTags::SetSystemsTagData(const FTagData* TagData)
{
	LLMCheck(GetGlobalNum() > 0);
	Values[0] = TagData;
}

const FTagData** FActiveTags::begin()
{
	return Values;
}

const FTagData* const* FActiveTags::begin() const
{
	return Values;
}

const FTagData* const* FActiveTags::end() const
{
	return Values + GetGlobalNum();
}

int32 FActiveTags::GetGlobalNum()
{
	return GetGlobalNumStorage();
}

void FActiveTags::SetGlobalNum(int32 NewNum)
{
	GetGlobalNumStorage() = NewNum;
}

FActiveTags FActiveTags::ConvertToPostCommandlineBootstrap(const FActiveTags& NewDefaults)
{
	FActiveTags Result;
	Result[0] = (*this)[0];
	for (int32 Index = 1; Index < GetGlobalNum(); ++Index)
	{
		Result[Index] = NewDefaults[Index];
	}
	return Result;
}

int32& FActiveTags::GetGlobalNumStorage()
{
	static int32 GlobalNum = 1;
	return GlobalNum;
}

uint32 GetTypeHash(const FActiveTags& Values)
{
	using namespace UE::LLMPrivate;

	// FTagData are never deleted, so we can use their pointer values
	constexpr uint32 Initial = 101;
	constexpr uint32 Multiplier = 103;
	uint32 Result = Initial;
	for (const FTagData* Ptr : Values)
	{
		Result = Result * Multiplier + ::GetTypeHash(Ptr);
	}
	return Result;
}

} // namespace UE::LLMPrivate

#endif // ENABLE_LOW_LEVEL_MEM_TRACKER