// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/LowLevelMemTracker.h"

#if ENABLE_LOW_LEVEL_MEM_TRACKER

#include "Templates/TypeHash.h"

namespace UE::LLMPrivate
{

/**
 * An embedded array of FTagData* that stores the active FTagData* for each of the FTagSet in LLM's global ordered
 * list of FTagSets. These arrays are snapshotted at the time of an allocation and then are stored as the 
 * key for the AllocationGroup shared by all allocations with that set tags.
 * The size in memory of the array is set at compile time, equal to NumPossibleTagSets*sizeof(Pointer).
 * The accessible length of the array is limited to NumActiveTagSets; the remaining values past the NumActiveTagSets
 * index are uninitialized, and are not read/written publicly or privately, with the exception of LLM bootstrapping.
 */
class FActiveTags
{
public:
	FActiveTags();
	FActiveTags(const FActiveTags& Other);
	FActiveTags(FActiveTags&& Other);
	FActiveTags& operator=(const FActiveTags& Other);
	FActiveTags& operator=(FActiveTags&& Other);

	const FTagData*& operator[](int32 Index);
	const FTagData*const& operator[](int32 Index) const;
	int32 Num() const;

	bool operator==(const FActiveTags& Other) const;

	/** Get/Set the TagData stored for ELLMTagSet::None in this array. Non-null for initialized FActiveTags. */
	const FTagData* GetSystemsTagData() const;
	void SetSystemsTagData(const FTagData* TagData);

	[[nodiscard]] const FTagData** begin();
	[[nodiscard]] const FTagData* const* begin() const;
	[[nodiscard]] const FTagData* const* end() const;

	static int32 GetGlobalNum();
	static void SetGlobalNum(int32 NewNum);

	FActiveTags ConvertToPostCommandlineBootstrap(const FActiveTags& NewDefaults);

private:
	static int32& GetGlobalNumStorage();

	/**
	 * Values in the array. The used length is <= compile time length, and is stored in GetGlobalNum().
	 * Before ActiveSets are parsed from commandline it is 1, after CommandLineBootstrapping it becomes constant and is
	 * equal to the number of active sets.
	 */
	const FTagData* Values[(uint8) ELLMTagSet::Max];
};

extern uint32 GetTypeHash(const UE::LLMPrivate::FActiveTags& Values);

} // namespace UE::LLMPrivate

#endif // ENABLE_LOW_LEVEL_MEM_TRACKER