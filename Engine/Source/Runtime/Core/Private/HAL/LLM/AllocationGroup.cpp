// Copyright Epic Games, Inc. All Rights Reserved.

#include "HAL/LLM/AllocationGroup.h"

#if ENABLE_LOW_LEVEL_MEM_TRACKER

namespace UE::LLMPrivate
{

FAllocationGroup::FAllocationGroup(int32 InIndexInAllocationGroups)
	: IndexInAllocationGroups(InIndexInAllocationGroups)
	, UpdatesSinceLastReference(0)
	, Initialized(0)
{
}

void FAllocationGroup::Initialize(const FActiveTags& InActiveTags)
{
	LLMCheck(InActiveTags[0] != nullptr); // ELLMTagSet::None should always be set

	ActiveTags = InActiveTags;
	for (const int64& Size : TrackerSizes)
	{
		LLMCheck(Size == 0);
	}
	LLMCheck(UpdatesSinceLastReference == 0);
	LLMCheck(ActiveAllocs == 0);
	LLMCheck(ThreadRefCount.load(std::memory_order_relaxed) == 0);

	Initialized = 1;
}

void FAllocationGroup::Reset()
{
	Initialized = 0;

	ActiveTags = FActiveTags();
	for (int64& Size : TrackerSizes)
	{
		Size = 0;
	}
	UpdatesSinceLastReference = 0;
	ActiveAllocs = 0;
	ThreadRefCount.store(0, std::memory_order_relaxed);
}

bool FAllocationGroup::IsInitialized() const
{
	return Initialized != 0;
}

int32 FAllocationGroup::GetIndex() const
{
	return IndexInAllocationGroups;
}

const FActiveTags& FAllocationGroup::GetActiveTags() const
{
	return ActiveTags;
}

void FAllocationGroup::ConvertToPostCommandlineBootstrap(const FActiveTags& NewDefaults)
{
	ActiveTags = ActiveTags.ConvertToPostCommandlineBootstrap(NewDefaults);
}

void FAllocationGroup::AddSizeAndAllocReferences(ELLMTracker Tracker, int64 DeltaSize, int32 DeltaAllocs,
	EPruningLevel PruningLevel)
{
	// Note that ActiveAllocs may sometimes go negative, because we are not taking accurate snapshots of it from
	// all threads, see the comment in FLLMTracker::FetchAndClearTagSizes.
	ActiveAllocs += DeltaAllocs;
	TrackerSizes[static_cast<uint8>(Tracker)] += DeltaSize;
	if (PruningLevel > EPruningLevel::None)
	{
		UpdatesSinceLastReference = 0;
	}
}

int64 FAllocationGroup::GetSize(ELLMTracker Tracker) const
{
	return TrackerSizes[static_cast<uint8>(Tracker)];
}

int32 FAllocationGroup::GetUpdatesSinceLastReference() const
{
	return static_cast<int32>(UpdatesSinceLastReference);
}

void FAllocationGroup::IncrementUpdatesSinceLastReference()
{
	// UpdatesSinceLastReference is a 16-bit integer to save space; don't let it overflow.
	// Our usage of it does not need an always-exact count, so just clamp it to max value if it reaches max value.
	if (UpdatesSinceLastReference < 0xffff)
	{
		++UpdatesSinceLastReference;
	}
}

void FAllocationGroup::ResetUpdatesSinceLastReference()
{
	UpdatesSinceLastReference = 0;
}

void FAllocationGroup::AddRefThread()
{
	ThreadRefCount.fetch_add(1, std::memory_order_relaxed);
}

void FAllocationGroup::ReleaseThread()
{
	int32 OldValue = ThreadRefCount.fetch_add(-1, std::memory_order_relaxed);
	LLMCheck(OldValue >= 1);
}

bool FAllocationGroup::IsReferenced() const
{
	if (ActiveAllocs != 0)
	{
		return true;
	}
	for (uint64 Size : TrackerSizes)
	{
		if (Size != 0)
		{
			return true;
		}
	}
	if (ThreadRefCount.load(std::memory_order_relaxed) != 0)
	{
		return true;
	}
	return false;
}

void FAllocationGroupTrackingData::ConditionalInitialize(FAllocationGroup* InGroup, bool bReferenceTrack)
{
	if (Group)
	{
		return;
	}
	LLMCheck(InGroup != nullptr);
	Group = InGroup;
	if (bReferenceTrack)
	{
		Group->AddRefThread();
	}
	Size = 0;
	ActiveAllocs = 0;
	UpdatesSinceLastAllocationChange = 0;
}

void FAllocationGroupTrackingData::ConditionalReset(bool bReferenceTrack)
{
	if (!Group)
	{
		return;
	}
	if (bReferenceTrack)
	{
		Group->ReleaseThread();
	}
	Group = nullptr;
}

bool FAllocationGroupTrackingData::IsInitialized() const
{
	return Group != nullptr;
}

FAllocationGroup* FAllocationGroupTrackingData::GetGroup() const
{
	return Group;
}

int64 FAllocationGroupTrackingData::GetSize() const
{
	return Size;
}

int32 FAllocationGroupTrackingData::GetActiveAllocs() const
{
	return ActiveAllocs;
}

void FAllocationGroupTrackingData::TrackAllocOrFree(int64 DeltaSize, int32 DeltaActiveAllocs)
{
	Size += DeltaSize;
	ActiveAllocs += DeltaActiveAllocs;
	UpdatesSinceLastAllocationChange = 0;
}

void FAllocationGroupTrackingData::DetachSizeAndActiveAllocs(FAllocationGroupTrackingData& Receiver, EPruningLevel PruningLevel)
{
	if (!Receiver.Group)
	{
		Receiver.Group = Group;
		Receiver.Size = 0;
		LLMCheck(Receiver.GetActiveAllocs() == 0);
		Receiver.ActiveAllocs = 0;
	}
	Receiver.Size += Size;
	Receiver.ActiveAllocs += ActiveAllocs;

	Size = 0;
	ActiveAllocs = 0;
	if (PruningLevel > EPruningLevel::None)
	{
		++UpdatesSinceLastAllocationChange;
	}
}

int32 FAllocationGroupTrackingData::GetUpdatesSinceLastAllocationChange() const
{
	return UpdatesSinceLastAllocationChange;
}

} // namespace UE::LLMPrivate


#endif // ENABLE_LOW_LEVEL_MEM_TRACKER