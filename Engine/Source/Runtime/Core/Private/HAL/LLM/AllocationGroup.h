// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/LowLevelMemTracker.h"

#if ENABLE_LOW_LEVEL_MEM_TRACKER

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "HAL/LLM/ActiveTags.h"
#include "HAL/LLM/LLMPrivateDefines.h"
#include "HAL/LLM/MemoryUtils.h"

#include <atomic>

namespace UE::LLMPrivate
{

/**
 * Records the list of tags that were active when a pointer was allocated, shared by the allocationinfo for all
 * pointers that had the same set of tags. AllocationGroups can be identified by their 4-byte index in a global
 * array, allowing us to minimize the memory overheaded necessary on the AllocationInfo to update tag sizes when
 * the allocation is freed. Allocation groups are discarded when no longer referenced by an active allocation (with
 * a few exceptions).
 * 
 * Records the size and number of active allocations made under that set of tags. These values are not precise due
 * to threading; when the values are updated we take snapshots of each thread's change in allocations outside of
 * a critical section, and a race condition may cause an allocation at the coarse-snapshot time to be temporarily
 * miscounted.
 * 
 * Sizes from all allocation groups with a given FTagData are summed to provide the size data for the TagData.
 */
class FAllocationGroup
{
public:
	FAllocationGroup(int32 InIndexInAllocationGroups);

	void Initialize(const FActiveTags& InActiveTags);
	void Reset();

	/** Initialized flag, used to implement a pool allocator for Allocation groups. */
	bool IsInitialized() const;
	/** Immutable data, this group's index in LLM's AllocationGroups. */
	int32 GetIndex() const;
	/**
	 * (Mostly) immutable data: the ActiveTags that this group is tracking.
	 * Changes only during CommandLineBootStrapping.
	 */
	const FActiveTags& GetActiveTags() const;

	/** Modifies otherwise immutable data, call only when all threads are paused by the global LLM EnableLock. */
	void ConvertToPostCommandlineBootstrap(const FActiveTags& NewDefaults);

	/**
	 * Update-thread only.
	 * Adds the accumulated change and Size and ActiveAllocs that were recorded by a thread over an update period.
	 * Also updates lifetime data for later pruning, if PruningLevel != EPruningLevel::None.
	 */
	void AddSizeAndAllocReferences(ELLMTracker Tracker, int64 DeltaSize, int32 DeltaAllocs, EPruningLevel PruningLevel);
	/** Update-thread only. Returns the currently known size of allocations in this group. */
	int64 GetSize(ELLMTracker Tracker) const;

	/** Update-thread only. Get/Modify the number of updates since the group was last referenced, used in pruning. */
	int32 GetUpdatesSinceLastReference() const;
	void IncrementUpdatesSinceLastReference();
	void ResetUpdatesSinceLastReference();

	/**
	 * Thread-safe increment of the reference counter on this. Reference count does not trigger destruction,
	 * it is informational only, does not have to include all references, and is used by the pruning code.
	 */
	void AddRefThread();
	/**
	 * Thread-safe decrement of the reference counter on this. Reference count does not trigger destruction,
	 * it is informational only, does not have to include all references, and is used by the pruning code.
	 */
	void ReleaseThread();
	/**
	 * Thread-safe report whether any threads or active allocations exist for *this.
	 * Note that due to threading race conditions it is possible for our ActiveAllocs field to incorrectly hold
	 * the value zero, but it is not possible for that to occur at the same time as ThreadRefCount == 0, so
	 * the combined boolean value of ThreadRefCount != 0 || ActiveAllocs != 0 is always accurate. See the comment
	 * in FLLMTracker::FetchAndClearTagSizes.
	 */ 
	bool IsReferenced() const;

private:
	FActiveTags ActiveTags;
	int64 TrackerSizes[static_cast<uint8>(ELLMTracker::Max)] = { 0 };
	int32 IndexInAllocationGroups = -1;
	uint32 UpdatesSinceLastReference : 16; // Initialized to 0
	uint32 Initialized : 1; // Initialized to 0
	int32 ActiveAllocs = 0;
	std::atomic<int32> ThreadRefCount{ 0 };
};

/**
 * Data held by a thread or during update that records the change in number and size of allocations under
 * an allocation group. All data is non-threadsafe, and should be guarded externally from simultaneous access
 * on multiple threads.
 */
class FAllocationGroupTrackingData
{
public:
	/**
	 * Initialize the TrackingData if not already initialized. Used when adding/removing TrackingDatas
	 * from threads and the update thread. bReferenceTrack is set to true when the TrackingData is stored
	 * on in the thread state for a thread other than the UpdateThread and therefore we need to record
	 * its reference to its AllocationGroup to prevent the group from being pruned.
	 */
	void ConditionalInitialize(FAllocationGroup* InGroup, bool bReferenceTrack = false);
	void ConditionalReset(bool bReferenceTrack = false);
	bool IsInitialized() const;

	/** The AllocationGroup tracked by this tracking data. Non-null when this TrackingData is initialized. */
	FAllocationGroup* GetGroup() const;

	/** The postive or negative change in allocation size since the last time Detach was called. */
	int64 GetSize() const;
	/** The postive or negative change in the number of ActiveAllocs size since the last time Detach was called. */
	int32 GetActiveAllocs() const;

	/** Record the sum of a number of allocations or frees. */
	void TrackAllocOrFree(int64 DeltaSize, int32 DeltaActiveAllocs);
	/** Add the Size and ActiveAllocs from this into the Receiver, and optionally update reference data for pruning. */
	void DetachSizeAndActiveAllocs(FAllocationGroupTrackingData& Receiver, EPruningLevel PruningLevel);

	/**
	 * Report the number of updates (calls to DetachSizeAndActiveAllocs) since the last time TrackAllocOrFree
	 * was called; used to prune the TrackingData.
	 */
	int32 GetUpdatesSinceLastAllocationChange() const;

private:
	FAllocationGroup* Group = nullptr;
	int64 Size = 0;
	int32 ActiveAllocs = 0;
	int32 UpdatesSinceLastAllocationChange = 0;
};

class FAllocationGroupArray : public TArray<FAllocationGroup*, FDefaultLLMAllocator>
{
public:
	using TArray<FAllocationGroup*, FDefaultLLMAllocator>::TArray;
};

class FAllocationGroupSet : public TSet<FAllocationGroup*, DefaultKeyFuncs<FAllocationGroup*>, FDefaultSetLLMAllocator>
{
public:
	using TSet<FAllocationGroup*, DefaultKeyFuncs<FAllocationGroup*>, FDefaultSetLLMAllocator>::TSet;
};

class FMapActiveTagsToAllocationGroup : public TMap<FActiveTags, FAllocationGroup*, FDefaultSetLLMAllocator>
{
public:
	using TMap<FActiveTags, FAllocationGroup*, FDefaultSetLLMAllocator>::TMap;
};

class FMapActiveTagsToAllocationGroupTrackingData : public TMap<FActiveTags, FAllocationGroupTrackingData, FDefaultSetLLMAllocator>
{
public:
	using TMap<FActiveTags, FAllocationGroupTrackingData, FDefaultSetLLMAllocator>::TMap;
};

} // namespace UE::LLMPrivate


#endif // ENABLE_LOW_LEVEL_MEM_TRACKER