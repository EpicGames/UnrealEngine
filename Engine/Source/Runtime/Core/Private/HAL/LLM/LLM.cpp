// Copyright Epic Games, Inc. All Rights Reserved.

#include "HAL/LowLevelMemTracker.h"

#include "Containers/ContainerAllocationPolicies.h"

#if ENABLE_LOW_LEVEL_MEM_TRACKER

#include "Async/SharedLock.h"
#include "Async/UniqueLock.h"
#include "HAL/FileManager.h"
#include "HAL/IConsoleManager.h"
#include "HAL/LLM/ActiveTags.h"
#include "HAL/LLM/AllocationGroup.h"
#include "HAL/LLM/LLMPrivate.h"
#include "HAL/LowLevelMemStats.h"
#include "HAL/ThreadHeartBeat.h"
#include "HAL/PlatformMemory.h" // for page allocation association.
#include "MemPro/MemProProfiler.h"
#include "Math/NumericLimits.h"
#include "Misc/CString.h"
#include "Misc/CoreDelegates.h"
#include "Misc/FileHelper.h"
#include "Misc/Fork.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Misc/Paths.h"
#include "Misc/ScopeExit.h"
#include "Misc/ScopeLock.h"
#include "Misc/ScopeRWLock.h"
#include "Misc/StringBuilder.h"
#include "Misc/VarArgs.h"
#include "Templates/Atomic.h"
#include "Trace/Trace.inl"

#if UE_ENABLE_ARRAY_SLACK_TRACKING

// Specifies whether to generate the whole log file in memory before writing.  Switch uses an async thread for file writing,
// which does array allocations, deadlocking on the critical section.  Writing to memory first avoids a dependency on the
// async thread, as we can write the memory to a file at the end, after the lock is released.
#define ARRAY_SLACK_LOG_TO_MEMORY !PLATFORM_WINDOWS

// Useful values to set in the debugger, to set a breakpoint on a particular allocation tag, element size, and max count.  Sometimes this can
// give more context regarding an allocation than what you get from a stack trace alone.  As an example, the constructor for UAudioCaptureComponent,
// which allocated a 1.83 MB array, just showed up as "UClass::CreateDefaultObject" in the stack trace.  The constructor for the specific subclass
// was optimized out, and there was no way to tell from the slack report what it was actually related to.  Stopping on the allocation in the debugger
// makes it immediately obvious, because you can see that the UClass in question is UAudioCaptureComponent.  These debug values also let you stop
// on places where the array count changes without triggering an allocation (the tracking only grabs call stacks on reallocation).
int32 GArraySlackTagToWatch = -1;		// (int32)ELLMTag::UObject;
int32 GArraySlackSizeToWatch = 0;
int32 GArraySlackMaxToWatch = 0;

uint32 GArraySlackDumpIndex = 0;				// Incremented for each auto-generated report file
bool GArraySlackInit = false;					// Set when we start tracking slack -- startup constructor allocations add a lot of noise (start this as true if you want these)
bool GArraySlackFirstStackOnly = true;			// Only do stack trace on first stack for a given allocation -- faster, but could be useful to know last allocation
bool GArraySlackGroupByTag = false;				// Group slack by tag when running with -llm
bool GArraySlackDefaultVerbose = true;			// Whether to default to verbose output, can be overridden with -Verbose=[0,1]

// We require a minimum number of total bytes for a group of allocations with the same stack trace to be reported.  A setting of 64 discards 80% of
// allocations representing less than 0.2% of the slack memory.  If you want to look at aggregations of smaller allocations, you can use -Stack=N to
// trim off more of the call stack, which will make more allocations alias to the same stack trace, and show up in the report (this is probably what
// you would do anyway when investigating that scenario).  Or you could locally set this to zero to get everything.
//
// Console platforms have far slower symbol lookup than PC, making slack reports take a couple orders of magnitude longer to generate than on PC,
// so we use more conservative settings.  Without some sort of trimming of the generated results, a slack report can take over an hour, which just
// isn't useful (it still can take 15 minutes with these settings, versus around 20 seconds on Windows for a much larger report).
#if PLATFORM_WINDOWS
static const int32 GArraySlackThreshold = 64;				// Typically covers 99.8% of slack memory
static const int32 GArraySlackFullStackNum = MAX_int32;		// Show full call stack context for all allocations
static const int32 GArraySlackDefaultStackDepth = 9;		// Sort by this deep in the call stack -- matches max call stack depth in FArraySlackTrackingHeader structure
#else
static const int32 GArraySlackThreshold = 8192;				// Typically covers 95% of slack memory
static const int32 GArraySlackFullStackNum = 150;			// Show full call stack context for this many allocations
static const int32 GArraySlackDefaultStackDepth = 5;		// Sort by this deep in the call stack
#endif

std::atomic<int64> GArrayMaxByTag[256];
std::atomic<int64> GArrayUsedByTag[256];
int64 GArraySlackByTag[256];
int32 GArrayCountByTag[256];

// Doubly linked list of all tracked allocations, and critical section to protect it.  Critical section is a pointer so we can detect if it's constructed,
// otherwise you get crashes in startup constructors which run before the constructor of the critical section is called.  Initialized in the LLM tracker,
// which gets initialized by the first startup code that uses an LLM scope (even when -llm is disabled), which tends to be pretty early -- allocations
// before that just won't be tracked.  If this became an issue, we could atomically initialize the lock on first access from any thread, but we already
// defer tracking until engine PreInit anyway to factor out noise from the myriad static global FString constructors, so it doesn't matter as it stands now.
FArraySlackTrackingHeader* GTrackArrayDetailedList;
FCriticalSection* GTrackArrayDetailedLock;

// Dummy function to set a breakpoint on where it's called, if you want to investigate code related to a certain allocation
FORCENOINLINE void LlmTrackSetBreakpointHere()
{
	// Need something non-empty that won't compile out
	static int32 Dummy = 0;
	Dummy++;
}

void FArraySlackTrackingHeader::AddAllocation()
{
	if (ArrayNum != INDEX_NONE)
	{
		GArrayMaxByTag[Tag].fetch_add(ArrayMax * (int64)ElemSize, std::memory_order_relaxed);
		GArrayUsedByTag[Tag].fetch_add(ArrayNum * (int64)ElemSize, std::memory_order_relaxed);

		// This code is only reached for reallocations, since during the initial allocation, ArrayNum won't have been set yet.
		ReallocCount++;
	}

	if (GArraySlackFirstStackOnly == false || NumStackFrames == 0)
	{
		// Skip the first 3 stack frames, which are tracking code (CaptureStackBackTrace, LlmTrackArrayAddAllocation, FArraySlackTrackingHeader::Realloc)
		constexpr int8 SkipStackFrames = 3;
		uint64 StackFrameTemp[UE_ARRAY_COUNT(StackFrames) + SkipStackFrames];
		NumStackFrames = (int8)FPlatformStackWalk::CaptureStackBackTrace(StackFrameTemp, UE_ARRAY_COUNT(StackFrameTemp)) - SkipStackFrames;
		if (NumStackFrames < 0)
		{
			NumStackFrames = 0;
		}
		for (int32 StackIndex = 0; StackIndex < NumStackFrames; StackIndex++)
		{
			StackFrames[StackIndex] = StackFrameTemp[SkipStackFrames + StackIndex];
		}
	}

	if (GTrackArrayDetailedLock && GArraySlackInit)
	{
		// For detailed tracking, we add the array header to a doubly linked list
		FScopeLock Lock(GTrackArrayDetailedLock);

		if (GTrackArrayDetailedList)
		{
			GTrackArrayDetailedList->Prev = &Next;
		}
		Next = GTrackArrayDetailedList;
		Prev = &GTrackArrayDetailedList;
		GTrackArrayDetailedList = this;

		if ((Tag == GArraySlackTagToWatch) &&
			(ElemSize == GArraySlackSizeToWatch) &&
			(!GArraySlackMaxToWatch || (ArrayMax == GArraySlackMaxToWatch)))
		{
			LlmTrackSetBreakpointHere();
		}

		GArrayCountByTag[Tag]++;
	}
}

void FArraySlackTrackingHeader::RemoveAllocation()
{
	if (ArrayNum != INDEX_NONE)
	{
		GArrayUsedByTag[Tag].fetch_sub(ArrayNum * (int64)ElemSize, std::memory_order_relaxed);
		GArrayMaxByTag[Tag].fetch_sub(ArrayMax * (int64)ElemSize, std::memory_order_release);
	}

	if (Prev)
	{
		FScopeLock Lock(GTrackArrayDetailedLock);

		GArrayCountByTag[Tag]--;

		if (Next)
		{
			Next->Prev = Prev;
		}
		(*Prev) = Next;

		Next = nullptr;
		Prev = nullptr;
	}
}

void FArraySlackTrackingHeader::UpdateNumUsed(int64 NewNumUsed)
{
	check(NewNumUsed <= ArrayMax);

	if ((Tag == GArraySlackTagToWatch) &&
		(ElemSize == GArraySlackSizeToWatch) &&
		(!GArraySlackMaxToWatch || (ArrayMax == GArraySlackMaxToWatch)))
	{
		LlmTrackSetBreakpointHere();
	}

	// Track the allocation in our totals when ArrayNum is first set to something other than INDEX_NONE.  This allows us to
	// factor out container allocations that aren't arrays (mainly hash tables), which won't ever call "UpdateNumUsed".
	if (ArrayNum == INDEX_NONE)
	{
		GArrayMaxByTag[Tag].fetch_add(ArrayMax * (int64)ElemSize, std::memory_order_relaxed);
		ArrayNum = 0;
		FirstAllocFrame = (uint32)GFrameCounter;
	}
	GArrayUsedByTag[Tag].fetch_add((NewNumUsed - ArrayNum) * (int64)ElemSize, std::memory_order_relaxed);
	ArrayNum = NewNumUsed;
	ArrayPeak = FMath::Max(ArrayPeak, (uint32)FMath::Min(NewNumUsed, 0xffffffffll));
}

FORCENOINLINE void* FArraySlackTrackingHeader::Realloc(void* Ptr, int64 Count, uint64 ElemSize, int32 Alignment)
{
	// Figure out how much padding we need under the allocation
	int32 HeaderAlign = FPlatformMath::RoundUpToPowerOfTwo(sizeof(FArraySlackTrackingHeader));
	int32 PaddingRequired = HeaderAlign > Alignment ? HeaderAlign : Alignment;

	// Get the base pointer of the original allocation, and remove tracking for it
	if (Ptr)
	{
		FArraySlackTrackingHeader* TrackingHeader = (FArraySlackTrackingHeader*)((uint8*)Ptr - sizeof(FArraySlackTrackingHeader));
		TrackingHeader->RemoveAllocation();

		Ptr = (uint8*)TrackingHeader - TrackingHeader->AllocOffset;
	}

	uint8* ResultPtr = nullptr;
	if (Count)
	{
		ResultPtr = (uint8*)FMemory::Realloc(Ptr, Count * ElemSize + PaddingRequired, Alignment);
		ResultPtr += PaddingRequired;
		FArraySlackTrackingHeader* TrackingHeader = (FArraySlackTrackingHeader*)(ResultPtr - sizeof(FArraySlackTrackingHeader));

		// Set the tag and other default information in the allocation if it's newly created
		if (!Ptr)
		{
			// Note that we initially set the slack tracking ArrayNum to INDEX_NONE.  The container allocator is used by both arrays and
			// other containers (Set / Map / Hash), and we don't know it's actually an array until "UpdateNumUsed" is called on it.
			check(PaddingRequired <= 65536);
			TrackingHeader->Next = nullptr;
			TrackingHeader->Prev = nullptr;
			TrackingHeader->AllocOffset = (uint16)(PaddingRequired - sizeof(FArraySlackTrackingHeader));
			TrackingHeader->Tag = LlmGetActiveTag();
			TrackingHeader->NumStackFrames = 0;			// Filled in later...
			TrackingHeader->FirstAllocFrame = 0;		// Filled in later...
			TrackingHeader->ReallocCount = 0;
			TrackingHeader->ArrayPeak = 0;
			TrackingHeader->ElemSize = ElemSize;
			TrackingHeader->ArrayNum = INDEX_NONE;		// Set in UpdateNumUsed
		}

		// Update ArrayMax and re-register the allocation
		TrackingHeader->ArrayMax = Count;
		TrackingHeader->AddAllocation();
	}
	else
	{
		FMemory::Free(Ptr);
	}

	return ResultPtr;
}

struct FArraySlackSortItem
{
	FArraySlackTrackingHeader* Header;
	FName CustomName;
	uint64 StackTraceTotalSlack;			// Sum of slack for elements with the same stack trace
	uint32 StackTraceRunLength;				// Run of items with the same stack trace
	uint32 RunLength;						// Run of identical elements (elemsize, num, max the same)
	int32 StackTraceIgnore;					// Number of stack trace items to ignore

	bool EqualsTagStackTrace(const FArraySlackSortItem& Other, uint32 StackTraceDepth, bool bLlmEnabled) const
	{
		if (bLlmEnabled)
		{
			if (Header->Tag != Other.Header->Tag)
			{
				return false;
			}
			if (CustomName != Other.CustomName)
			{
				return false;
			}
		}

		// If the stack depth is set to zero, and LLM is disabled, basically everything in the capture will get lumped into
		// one bucket.  Comparing by ElemSize is a last resort to force some differentiation in the report in that case (or
		// perhaps if we have a platform that doesn't support stack traces, or someone wants to locally disable them for
		// performance).  In cases where a stack frame exists, the leaf stack frame is always some sort of template type
		// specific resize (i.e. TArray<float>::ResizeTo), so this would be redundant, but also harmless.
		if (Header->ElemSize != Other.Header->ElemSize)
		{
			return false;
		}

		uint32 NumStackFramesThis = FMath::Min((uint32)(Header->NumStackFrames - StackTraceIgnore), StackTraceDepth);
		uint32 NumStackFramesOther = FMath::Min((uint32)(Other.Header->NumStackFrames - Other.StackTraceIgnore), StackTraceDepth);

		if (NumStackFramesThis != NumStackFramesOther)
		{
			return false;
		}

		return FMemory::Memcmp(&Header->StackFrames[StackTraceIgnore], &Other.Header->StackFrames[Other.StackTraceIgnore], NumStackFramesThis * sizeof(Header->StackFrames[0])) == 0;
	}

	bool Compare(const FArraySlackSortItem& Other, uint32 StackTraceDepth, bool bLlmEnabled) const
	{
		// Order by decreasing stack trace slack total bytes
		if (StackTraceTotalSlack != Other.StackTraceTotalSlack)
		{
			return StackTraceTotalSlack > Other.StackTraceTotalSlack;
		}

		// Order by tag
		if (bLlmEnabled)
		{
			if (Header->Tag != Other.Header->Tag)
			{
				return Header->Tag < Other.Header->Tag;
			}
			if (CustomName != Other.CustomName)
			{
				return CustomName.GetComparisonIndex().CompareFast(Other.CustomName.GetComparisonIndex()) < 0;
			}
		}

		// Order by increasing element size
		if (Header->ElemSize != Other.Header->ElemSize)
		{
			return Header->ElemSize < Other.Header->ElemSize;
		}

		// Order by stack trace
		uint32 NumStackFramesThis = FMath::Min((uint32)(Header->NumStackFrames - StackTraceIgnore), StackTraceDepth);
		uint32 NumStackFramesOther = FMath::Min((uint32)(Other.Header->NumStackFrames - Other.StackTraceIgnore), StackTraceDepth);

		if (NumStackFramesThis != NumStackFramesOther)
		{
			return NumStackFramesThis < NumStackFramesOther;
		}
		int32 StackFrameOrdinalCompare = memcmp(&Header->StackFrames[StackTraceIgnore], &Other.Header->StackFrames[Other.StackTraceIgnore], NumStackFramesThis * sizeof(Header->StackFrames[0]));
		if (StackFrameOrdinalCompare)
		{
			return StackFrameOrdinalCompare < 0;
		}

		int64 SlackBytesA = Header->SlackSizeInBytes();
		int64 SlackBytesB = Other.Header->SlackSizeInBytes();
		int64 RunSlackBytesA = SlackBytesA * RunLength;
		int64 RunSlackBytesB = SlackBytesB * Other.RunLength;

		// Order by decreasing run slack total bytes
		if (RunSlackBytesA != RunSlackBytesB)
		{
			return RunSlackBytesA > RunSlackBytesB;
		}

		// Order by decreasing individual item slack bytes
		if (SlackBytesA != SlackBytesB)
		{
			return SlackBytesA > SlackBytesB;
		}

		// Order by increasing Max
		return Header->ArrayMax < Other.Header->ArrayMax;
	}
};

static void LlmTrackArrayDumpTag(FArchive* LogFile, FOutputDevice& Output, TAnsiStringBuilder<4096>& Builder, int32 TagIndex, uint32 StackTraceDepth, bool bVerbose, double StartTime)
{
	FLLMGlobals& Tracker = FLLMGlobals::GetInner();
	bool bLlmEnabled = Tracker.IsEnabled();

	FScopeLock Lock(GTrackArrayDetailedLock);

	TArray<FArraySlackSortItem> ArraySlackSortArray;
	int32 ReserveAmount = 0;
	if (TagIndex == INDEX_NONE)
	{
		for (int32 CountByTag : GArrayCountByTag)
		{
			ReserveAmount += CountByTag;
		}
	}
	else
	{
		ReserveAmount = GArrayCountByTag[TagIndex];
	}
	ArraySlackSortArray.Reserve(ReserveAmount);
	ArraySlackSortArray.GetAllocatorInstance().DisableSlackTracking();

	for (FArraySlackTrackingHeader* Current = GTrackArrayDetailedList; Current; Current = Current->Next)
	{
		// Don't bother dumping allocations with zero waste (or untracked where ArrayNum == INDEX_NONE)
		if ((TagIndex == INDEX_NONE || Current->Tag == TagIndex) && (Current->ArrayNum != INDEX_NONE) && (Current->ArrayNum != Current->ArrayMax))
		{
			FArraySlackSortItem& SortItem = ArraySlackSortArray.AddDefaulted_GetRef();

			SortItem.Header = Current;
			if (bLlmEnabled && (Current->Tag == (uint8)ELLMTag::CustomName))
			{
				SortItem.CustomName = Tracker.Get().FindPtrDisplayName((uint8*)Current - Current->AllocOffset);
			}
			else
			{
				SortItem.CustomName = NAME_None;
			}

			// Filled in later
			SortItem.StackTraceTotalSlack = 0;
			SortItem.StackTraceRunLength = 1;
			SortItem.RunLength = 1;
			SortItem.StackTraceIgnore = 0;
		}
	}

	// We want to ignore ResizeAllocation() if it's the first stack frame, as it's not interesting.  This stack frame will sometimes be there,
	// and sometimes not, because the most common ResizeAllocation() template variations are tagged FORCENOINLINE to reduce code size.  We set
	// StackTraceIgnore=1 to indicate where this is the first stack frame, indicating we can ignore it downstream.  It has to be filled in before
	// the sort, to properly handle the StackTraceDepth setting.
	//
	// Assuming symbol lookups are expensive, we sort by leaf stack frame first so we only need to do symbol lookups for unique leaf stack frames.
	Algo::SortBy(ArraySlackSortArray, [](const FArraySlackSortItem& Item) { return Item.Header->StackFrames[0]; });

	for (int32 ItemIndex = 0; ItemIndex < ArraySlackSortArray.Num(); ItemIndex++)
	{
		int32 StackTraceIgnore = 0;
		if (ItemIndex == 0 || ArraySlackSortArray[ItemIndex].Header->StackFrames[0] != ArraySlackSortArray[ItemIndex - 1].Header->StackFrames[0])
		{
			StackTraceIgnore = 0;
			if (ArraySlackSortArray[ItemIndex].Header->NumStackFrames)
			{
				FProgramCounterSymbolInfo SymbolInfo;
				FPlatformStackWalk::ProgramCounterToSymbolInfo(ArraySlackSortArray[ItemIndex].Header->StackFrames[0], SymbolInfo);
				if (FCStringAnsi::Strstr(SymbolInfo.FunctionName, "::ResizeAllocation("))
				{
					StackTraceIgnore = 1;
				}
			}
		}

		ArraySlackSortArray[ItemIndex].StackTraceIgnore = StackTraceIgnore;
	}

	// First pass sort -- we haven't yet filled in totals for StackTraceTotalSlack and RunTotalSlack
	Algo::Sort(ArraySlackSortArray, [StackTraceDepth, bLlmEnabled](const FArraySlackSortItem& A, const FArraySlackSortItem& B) { return A.Compare(B, StackTraceDepth, bLlmEnabled); });

	int64 IgnoredGroups = 0;
	int64 IgnoredSlack = 0;
	{
		// Compute slack associated with each stack trace and runs of identical elements, and store it on the sort elements
		int32 StackTraceRun = 0;
		int64 StackTraceTotal = 0;
		uint32 ElementRun = 0;
		for (int32 ItemIndex = 0; ItemIndex < ArraySlackSortArray.Num(); ItemIndex++)
		{
			// Add current item
			int64 ElementSlack = ArraySlackSortArray[ItemIndex].Header->SlackSizeInBytes();
			StackTraceRun++;
			StackTraceTotal += ElementSlack;
			ElementRun++;

			// If the next item has a different stack trace, or it's the end of the array, echo the stack trace total slack to all the items
			if ((ItemIndex == ArraySlackSortArray.Num() - 1) || !ArraySlackSortArray[ItemIndex].EqualsTagStackTrace(ArraySlackSortArray[ItemIndex + 1], StackTraceDepth, bLlmEnabled))
			{
				for (int32 RunIndex = ItemIndex - (StackTraceRun - 1); RunIndex <= ItemIndex; RunIndex++)
				{
					ArraySlackSortArray[RunIndex].StackTraceTotalSlack = StackTraceTotal;
					ArraySlackSortArray[RunIndex].StackTraceRunLength = StackTraceRun;
				}
				if (StackTraceTotal < GArraySlackThreshold)
				{
					IgnoredGroups++;
					IgnoredSlack += StackTraceTotal;
				}
				StackTraceTotal = 0;
				StackTraceRun = 0;
			}

			// Check if the item is different at all, and echo run length to all the items
			if ((ItemIndex == ArraySlackSortArray.Num() - 1) || ArraySlackSortArray[ItemIndex].Compare(ArraySlackSortArray[ItemIndex + 1], StackTraceDepth, bLlmEnabled))
			{
				for (int32 RunIndex = ItemIndex - (ElementRun - 1); RunIndex <= ItemIndex; RunIndex++)
				{
					ArraySlackSortArray[RunIndex].RunLength = ElementRun;
				}
				ElementRun = 0;
			}
		}
	}

	// Second pass, final sort
	Algo::Sort(ArraySlackSortArray, [StackTraceDepth, bLlmEnabled](const FArraySlackSortItem& A, const FArraySlackSortItem& B) { return A.Compare(B, StackTraceDepth, bLlmEnabled); });

	// Only include tag column if LLM is enabled
	const TCHAR* TagColumnSeparator = bLlmEnabled ? TEXT("\t") : TEXT("");
	const ANSICHAR* TagColumnSeparatorANSI = bLlmEnabled ? "\t" : "";

	if (LogFile)
	{
		Builder.Reset();
		Builder.Appendf("Ignored:\t%lld\tGroups,\t%lld\tBytes total\n", IgnoredGroups, IgnoredSlack);
		Builder.Appendf("Under:\t%d\tGroup size threshold\n\n", GArraySlackThreshold);

		Builder.Appendf("NumArrays\tReallocs\tLifetime\tPeakAvg\tPeak\tElemSize\tNum\tMax\t%s\tStackSlack%s%s\tStackTrace\n",
			bVerbose ? "ItemSlack" : "LargestItem",
			TagColumnSeparatorANSI,
			bLlmEnabled ? "Tag" : "");
		LogFile->Serialize(Builder.GetData(), Builder.Len());
	}
	else
	{
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Ignored:\t%lld\tGroups,\t%lld\tBytes total\n"), IgnoredGroups, IgnoredSlack);
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Under:\t%d\tGroup size threshold\n\n"), GArraySlackThreshold);

		// We prepend "SlackReport" to every line when outputting to the debug window, as this can help filtering out random
		// log lines after the output is cut and pasted.  You can sort by the first column to accomplish that.
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("SlackReport\tNumArrays\tReallocs\tLifetime\tPeakAvg\tPeak\tElemSize\tNum\tMax\t%s\tStackSlack%s%s\tStackTrace\n"),
			bVerbose ? "ItemSlack" : "LargestItem",
			TagColumnSeparator,
			bLlmEnabled ? TEXT("Tag") : TEXT(""));
	}

	{
		uint32 CurrentFrame = (uint32)GFrameCounter;
		int32 ItemRun = 0;
		double ItemReallocs = 0.0;
		double ItemLifetime = 0.0;
		double ItemPeakSum = 0.0;
		uint32 ItemPeak = 0;
		int32 FullStacksPrinted = 0;

		for (int32 ItemIndex = 0; ItemIndex < ArraySlackSortArray.Num(); ItemIndex++)
		{
#if PLATFORM_WINDOWS
			// Windows is a lot faster, so we don't need as much progress logging
			constexpr int32 ProgressInterval = 25000;
#else
			constexpr int32 ProgressInterval = 1000;
#endif
			if ((ItemIndex % ProgressInterval) == 0)
			{
				Output.Logf(TEXT("Array Slack %d / %d allocs...  (%.2lf minutes, batch %lld bytes -> threshold %lld)"),
					ItemIndex, ArraySlackSortArray.Num(), (FPlatformTime::Seconds() - StartTime) / 60.0, ArraySlackSortArray[ItemIndex].StackTraceTotalSlack, GArraySlackThreshold);
			}

			// Count current item
			ItemRun++;
			ItemReallocs += ArraySlackSortArray[ItemIndex].Header->ReallocCount;
			ItemLifetime += CurrentFrame - ArraySlackSortArray[ItemIndex].Header->FirstAllocFrame;
			ItemPeakSum += ArraySlackSortArray[ItemIndex].Header->ArrayPeak;
			ItemPeak = FMath::Max(ItemPeak, ArraySlackSortArray[ItemIndex].Header->ArrayPeak);

			// If this item is different than the next, or the last item, echo the count and item
			if ((ItemIndex == ArraySlackSortArray.Num() - 1) || ArraySlackSortArray[ItemIndex].Compare(ArraySlackSortArray[ItemIndex + 1], StackTraceDepth, bLlmEnabled))
			{
				// Determine if this run has the same stack trace as the previous.  In verbose mode, we only print stack trace specific totals for unique
				// stack traces, and when not verbose, we only print lines at all for unique stack traces.
				int32 RunStart = ItemIndex - (ItemRun - 1);
				bool bUniqueStackTrace = RunStart == 0 || !ArraySlackSortArray[RunStart - 1].EqualsTagStackTrace(ArraySlackSortArray[RunStart], StackTraceDepth, bLlmEnabled);

				if ((bVerbose || bUniqueStackTrace) && (ArraySlackSortArray[ItemIndex].StackTraceTotalSlack >= GArraySlackThreshold))
				{
					if (LogFile)
					{
						Builder.Reset();
						Builder.Appendf(
							"%llu\t%.1lf\t%.1lf\t%.1lf\t%u\t%lld\t%lld\t%lld\t%lld\t",
							bVerbose ? ItemRun : ArraySlackSortArray[ItemIndex].StackTraceRunLength,
							ItemReallocs / ItemRun,
							ItemLifetime / ItemRun,
							ItemPeakSum / ItemRun,
							ItemPeak,
							ArraySlackSortArray[ItemIndex].Header->ElemSize,
							ArraySlackSortArray[ItemIndex].Header->ArrayNum,
							ArraySlackSortArray[ItemIndex].Header->ArrayMax,
							ArraySlackSortArray[ItemIndex].Header->SlackSizeInBytes() * ItemRun);

						// Stack trace total slack if it's unique
						if (bUniqueStackTrace)
						{
							Builder.Appendf("%lld", ArraySlackSortArray[ItemIndex].StackTraceTotalSlack);
						}

						// Tag name
						if (bLlmEnabled)
						{
							Builder.AppendChar('\t');
							if (ArraySlackSortArray[ItemIndex].CustomName != NAME_None)
							{
								Builder.Append(*ArraySlackSortArray[ItemIndex].CustomName.ToString());
							}
							else
							{
								Builder.Append(Tracker.FindTagDisplayName(ArraySlackSortArray[ItemIndex].Header->Tag).ToString());
							}
						}
					}
					else
					{
						TStringBuilder<32> ByStackSlack;
						if (bUniqueStackTrace)
						{
							ByStackSlack.Appendf(TEXT("%lld"), ArraySlackSortArray[ItemIndex].StackTraceTotalSlack);
						}

						FPlatformMisc::LowLevelOutputDebugStringf(
							TEXT("SlackReport\t%llu\t%.1lf\t%.1lf\t%.1lf\t%lld\t%lld\t%lld\t%lld\t%lld\t%s%s%s"),
							bVerbose ? ItemRun : ArraySlackSortArray[ItemIndex].StackTraceRunLength,
							ItemReallocs / ItemRun,
							ItemLifetime / ItemRun,
							ItemPeakSum / ItemRun,
							ItemPeak,
							ArraySlackSortArray[ItemIndex].Header->ElemSize,
							ArraySlackSortArray[ItemIndex].Header->ArrayNum,
							ArraySlackSortArray[ItemIndex].Header->ArrayMax,
							ArraySlackSortArray[ItemIndex].Header->SlackSizeInBytes() * ItemRun,
							ByStackSlack.ToString(),
							TagColumnSeparator,
							!bLlmEnabled ? TEXT("") :
							(ArraySlackSortArray[ItemIndex].CustomName != NAME_None ?
								*ArraySlackSortArray[ItemIndex].CustomName.ToString() :
								*Tracker.FindTagDisplayName(ArraySlackSortArray[ItemIndex].Header->Tag).ToString()));
					}

					// Only print stack trace if this is a unique stack trace.
					if (bUniqueStackTrace)
					{
						int32 StackFirst = ArraySlackSortArray[ItemIndex].StackTraceIgnore;
						int32 StackCount = ArraySlackSortArray[ItemIndex].Header->NumStackFrames - ArraySlackSortArray[ItemIndex].StackTraceIgnore;
						if (FullStacksPrinted >= GArraySlackFullStackNum)
						{
							StackCount = FMath::Min(StackCount, (int32)StackTraceDepth);
						}
						else
						{
							FullStacksPrinted++;
						}

						for (int32 StackIndex = StackFirst; StackIndex < StackFirst + StackCount; StackIndex++)
						{
							FProgramCounterSymbolInfo SymbolInfo;
							FPlatformStackWalk::ProgramCounterToSymbolInfo(ArraySlackSortArray[ItemIndex].Header->StackFrames[StackIndex], SymbolInfo);

							if (LogFile)
							{
								Builder.AppendChar('\t');
								Builder.Append(SymbolInfo.FunctionName[0] ? SymbolInfo.FunctionName : "UnknownFunction");

								if (SymbolInfo.Filename[0] && SymbolInfo.LineNumber)
								{
									// Format " [Filename:Line]"
									Builder.Append(" [");
									Builder.Append(SymbolInfo.Filename);
									Builder.Appendf(":%i]", SymbolInfo.LineNumber);
								}
								else
								{
									Builder.Append(" []");
								}
							}
							else
							{
								TStringBuilder<512> SymbolName;
								SymbolName.AppendChar(TEXT('\t'));
								SymbolName.Append(SymbolInfo.FunctionName[0] ? SymbolInfo.FunctionName : "UnknownFunction");

								if (SymbolInfo.Filename[0] && SymbolInfo.LineNumber)
								{
									// Format " [Filename:Line]"
									SymbolName.Append(TEXT(" ["));
									SymbolName.Append(SymbolInfo.Filename);
									SymbolName.Appendf(TEXT(":%i]"), SymbolInfo.LineNumber);
								}
								else
								{
									SymbolName.Append(TEXT(" []"));
								}

								FPlatformMisc::LowLevelOutputDebugString(SymbolName.ToString());
							}
						}
					}

					if (LogFile)
					{
						Builder.AppendChar('\n');
						LogFile->Serialize(Builder.GetData(), Builder.Len());
					}
					else
					{
						FPlatformMisc::LowLevelOutputDebugString(TEXT("\n"));
					}
				}

				ItemRun = 0;
				ItemReallocs = 0.0;
				ItemLifetime = 0.0;
				ItemPeakSum = 0.0;
				ItemPeak = 0;
			}
		}
	}
}

void ArraySlackTrackInit()
{
	// Any array allocations before this is called won't have array slack tracking, although subsequent reallocations of existing arrays
	// will gain tracking if that occurs.  The goal is to filter out startup constructors which run before Main, which introduce a
	// ton of noise into slack reports.  Especially the roughly 30,000 static FString constructors in the code base, each with a
	// unique call stack, and all having a little bit of slack due to malloc bucket size rounding.
	GArraySlackInit = true;
}

static void LlmTrackArrayTick()
{
	// Updating these every frame is handy, so you can see them in a watch window in the debugger or debug print them without running a capture.
	// We could consider including these as stats, so you can track them in Insights, but the report is giving enough information for now.
	for (int32 TagIndex = 0; TagIndex < 256; TagIndex++)
	{
		GArraySlackByTag[TagIndex] = GArrayMaxByTag[TagIndex] - GArrayUsedByTag[TagIndex];
	}
}

void ArraySlackTrackGenerateReport(const TCHAR* Cmd, FOutputDevice& Output)
{
	Output.Logf(TEXT("Generating Array Slack report."));

	double StartTime = FPlatformTime::Seconds();

	// Make sure the slack by tag totals are up to date
	LlmTrackArrayTick();

	// Parse command -- tokens
	FString LogFilename;
	uint32 StackTraceDepth = GArraySlackDefaultStackDepth;
	bool bVerbose = GArraySlackDefaultVerbose;
	for (FString Arg = FParse::Token(Cmd, false); !Arg.IsEmpty(); Arg = FParse::Token(Cmd, false))
	{
		if (Arg[0] == TEXT('-'))
		{
			FStringView StackDepthSwitch(TEXTVIEW("-Stack="));
			FStringView VerboseSwitch(TEXTVIEW("-Verbose="));
			if ((Arg.Len() > StackDepthSwitch.Len()) && !FCString::Strnicmp(&Arg[0], StackDepthSwitch.GetData(), StackDepthSwitch.Len()))
			{
				StackTraceDepth = (uint32)FCString::Strtoui64(&Arg[StackDepthSwitch.Len()], nullptr, 10);
			}
			else if ((Arg.Len() > VerboseSwitch.Len()) && !FCString::Strnicmp(&Arg[0], VerboseSwitch.GetData(), VerboseSwitch.Len()))
			{
				bVerbose = Arg[VerboseSwitch.Len()] != TEXT('0');
			}
			else
			{
				Output.Logf(TEXT("Array Slack unsupported switch: \"%s\".  Valid switches:  -Stack=N, -Verbose=[0,1]"), *Arg);
			}
		}
		else
		{
			LogFilename = Arg;
		}
	}

	FArchive* LogFile = nullptr;

#if ARRAY_SLACK_LOG_TO_MEMORY
	TArray<uint8> LogFileMemory;
	LogFileMemory.Reserve(4 * 1024 * 1024);
	LogFileMemory.GetAllocatorInstance().DisableSlackTracking();
#endif
	FString LogFilenameWithPath;

	// Special name "NOFILE" indicates to write to debug log instead of file.  Useful for debugging the system, as you can see the
	// lines of text generated while debugging, as opposed to needing to wait until the file gets written to look at the output.
	if (!LogFilename.Equals(TEXT("NOFILE")))
	{
		FString AbsoluteProjectLogDir = FPaths::ConvertRelativePathToFull(FPaths::ProjectLogDir());
		FString SlackReportLogDir = FPaths::Combine(AbsoluteProjectLogDir, TEXT("SlackReport"));
		IPlatformFile::GetPlatformPhysical().CreateDirectoryTree(*SlackReportLogDir);

		if (LogFilename.IsEmpty())
		{
			LogFilename = FString::Printf(TEXT("SlackDump_%03u.tsv"), GArraySlackDumpIndex++);
		}
		else
		{
			if (!LogFilename.EndsWith(TEXT(".tsv")))
			{
				LogFilename.Append(TEXT(".tsv"));
			}
		}
		LogFilenameWithPath = SlackReportLogDir / LogFilename;

#if ARRAY_SLACK_LOG_TO_MEMORY
		LogFile = new FMemoryWriter(LogFileMemory);
#else
		IFileManager* FileManager = &IFileManager::Get();
		LogFile = FileManager->CreateFileWriter(*LogFilenameWithPath, 0);
#endif
	}

	const FLLMGlobals& Tracker = FLLMGlobals::GetInner();
	bool bLlmEnabled = Tracker.IsEnabled();

	TAnsiStringBuilder<4096> Builder;

	TArray<uint64> SortedTags;
	SortedTags.SetNumZeroed(256);
	SortedTags.GetAllocatorInstance().DisableSlackTracking();

	// Tag summary report is only useful when -llm is enabled
	if (bLlmEnabled)
	{
		if (LogFile)
		{
			Builder.Append("Tag\tSlack\tTotalMem\n");
			LogFile->Serialize(Builder.GetData(), Builder.Len());
		}
		else
		{
			FPlatformMisc::LowLevelOutputDebugString(TEXT("SlackByTag\tTag\tSlack\tTotalMem\n"));
		}

		// Sort tags by descending memory.  Store tag index in low 8 bits, memory in high 56 bits.
		for (int32 TagIndex = 0; TagIndex < 256; TagIndex++)
		{
			SortedTags[TagIndex] = (GArraySlackByTag[TagIndex] << 8) | TagIndex;
		}
		SortedTags.Sort(TGreater<uint64>());

		for (int32 SortedIndex = 0; SortedIndex < 256; SortedIndex++)
		{
			if (SortedTags[SortedIndex] >= 256)
			{
				uint32 TagIndex = (uint32)SortedTags[SortedIndex] & 0xff;

				if (LogFile)
				{
					Builder.Reset();
					Builder.Append(Tracker.FindTagDisplayName(TagIndex).ToString());
					Builder.Appendf("\t%lld\t%lld\n",
						GArraySlackByTag[TagIndex],
						GArrayMaxByTag[TagIndex].load(std::memory_order_relaxed));
					LogFile->Serialize(Builder.GetData(), Builder.Len());
				}
				else
				{
					FPlatformMisc::LowLevelOutputDebugStringf(TEXT("SlackByTag\t%s\t%lld\t%lld\n"),
						*Tracker.FindTagDisplayName(TagIndex).ToString(),
						GArraySlackByTag[TagIndex],
						GArrayMaxByTag[TagIndex].load(std::memory_order_relaxed));
				}
			}
		}

		if (LogFile)
		{
			char NewLines[] = "\n\n";
			LogFile->Serialize(NewLines, 2);
		}
	}

	if (LogFile)
	{
		// Append information about options and timing of dump
		FArraySlackTrackingHeader Dummy;
		Builder.Reset();
		Builder.Appendf("Ran with:\t-Stack=%u -Verbose=%d", FMath::Min((uint32)UE_ARRAY_COUNT(Dummy.StackFrames), StackTraceDepth), bVerbose ? 1 : 0);
		if (FLowLevelMemTracker::Get().IsEnabled())
		{
			Builder.Append(" -llm");
		}
		if (!bVerbose)
		{
			Builder.Append("\t\t\tFields besides NumArrays / StackSlack are for the largest slack bucket (unique Num / Max combo), run with -Verbose=1 to see all buckets");
		}
		Builder.Appendf("\nOn frame:\t%u\n\n", (int32)GFrameCounter);
		LogFile->Serialize(Builder.GetData(), Builder.Len());
	}

	if (bLlmEnabled && GArraySlackGroupByTag)
	{
		// Original behavior grouped by tag, but all in one batch is generally preferable.  Could expose this with a switch in the future.
		for (int32 SortedIndex = 0; SortedIndex < 256; SortedIndex++)
		{
			if (SortedTags[SortedIndex] > 256)
			{
				int32 TagIndex = (int32)SortedTags[SortedIndex] & 0xff;
				if (GArraySlackByTag[TagIndex])
				{
#if ARRAY_SLACK_LOG_TO_MEMORY
					// Disable tracking each loop iteration, in case the memory grew to the point where it was reallocated in the previous iteration.
					LogFileMemory.GetAllocatorInstance().DisableSlackTracking();
#endif

					LlmTrackArrayDumpTag(LogFile, Output, Builder, TagIndex, StackTraceDepth, bVerbose, StartTime);
				}
			}
		}
	}
	else
	{
		// INDEX_NONE == dump all tags in one batch
		LlmTrackArrayDumpTag(LogFile, Output, Builder, INDEX_NONE, StackTraceDepth, bVerbose, StartTime);
	}

	if (LogFile)
	{
		delete LogFile;

#if ARRAY_SLACK_LOG_TO_MEMORY
		// Now create the actual file
		IFileManager* FileManager = &IFileManager::Get();
		LogFile = FileManager->CreateFileWriter(*LogFilenameWithPath, 0);
		LogFile->Serialize(LogFileMemory.GetData(), LogFileMemory.Num());
		delete LogFile;
#endif
	}

	Output.Logf(TEXT("Finished generating Array Slack report to %s."), LogFile ? *LogFilename : TEXT("[Debug Output]"));
}

uint8 LlmGetActiveTag()
{
	FLLMGlobals& Tracker = FLLMGlobals::GetInner();
	if (!Tracker.IsInitialized())
	{
		return 0;
	}

	const UE::LLMPrivate::FTagData* TagData = Tracker.GetActiveTagData(ELLMTracker::Default);
	return TagData ? (uint8)TagData->GetEnumTag() : 0;
}

#endif  // UE_ENABLE_ARRAY_SLACK_TRACKING

UE_TRACE_CHANNEL(MemTagChannel,
	"LLM tag hierarchy and per-tracker per-tag memory values; "
	"drives the Memory Tags view in Memory Insights (enabling this channel auto-enables LLM).")

UE_TRACE_EVENT_BEGIN(LLM, TagsSpec, NoSync|Important)
	UE_TRACE_EVENT_FIELD(uint64, TagId)
	UE_TRACE_EVENT_FIELD(uint64, ParentId)
	UE_TRACE_EVENT_FIELD(uint8, TagSetId)
	UE_TRACE_EVENT_FIELD(UE::Trace::AnsiString, Name)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(LLM, TrackerSpec, NoSync|Important)
	UE_TRACE_EVENT_FIELD(uint8, TrackerId)
	UE_TRACE_EVENT_FIELD(UE::Trace::AnsiString, Name)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(LLM, TagSetSpec, NoSync|Important)
	UE_TRACE_EVENT_FIELD(uint8, TagSetId)
	UE_TRACE_EVENT_FIELD(UE::Trace::AnsiString, Name)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(LLM, TagValue)
	UE_TRACE_EVENT_FIELD(uint8, TrackerId)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(uint64[], Tags)
	UE_TRACE_EVENT_FIELD(int64[], Values)
UE_TRACE_EVENT_END()

#if LLM_CSV_PROFILER_WRITER_ENABLED
	CSV_DEFINE_CATEGORY(LLM, true);
	CSV_DEFINE_CATEGORY(LLMPlatform, true);
	CSV_DEFINE_CATEGORY(LLMCodeOrContent, true);
#endif

TAutoConsoleVariable<int32> CVarLLMTrackPeaks(TEXT("LLM.TrackPeaks"), 0,
	TEXT("Track peak memory in each category since process start rather than current frame's value."));

TAutoConsoleVariable<int32> CVarLLMWriteInterval(
	TEXT("LLM.LLMWriteInterval"),
	1,
	TEXT("The number of seconds between each line in the LLM csv (zero to write every frame)")
);

TAutoConsoleVariable<int32> CVarLLMCsvFlushEveryRow(TEXT("LLM.CsvFlushEveryRow"), 1,
	TEXT("Whether to flush the CSV with every row written. If disabled we only flush on a crash"));

TAutoConsoleVariable<int32> CVarLLMHeaderMaxSize(
	TEXT("LLM.LLMHeaderMaxSize"),
	-1,
	TEXT("The maximum total number of characters allowed for all of the LLM TagNames, or -1 to calculate it from all ELLMTagSet::None tags.")
);

FAutoConsoleCommand LLMSnapshot(
	TEXT("LLMSnapshot"),
	TEXT("Takes a single LLM Snapshot of one frame. This command requires the commandline -llmdisableautopublish"),
	FConsoleCommandDelegate::CreateLambda([]()
	{
		FLowLevelMemTracker::Get().PublishDataSingleFrame();
	}));

FAutoConsoleCommand DumpLLM(
	TEXT("DumpLLM"),
	TEXT("Logs out the current and peak sizes of all tracked LLM tags"),
	FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateLambda(
		[](const TArray<FString>& Args, UWorld* InWorld, FOutputDevice& Ar)
	{
		FString Command = FString::Join(Args, TEXT(" "));

		FLowLevelMemTracker::EDumpFormat DumpFormat = FLowLevelMemTracker::EDumpFormat::PlainText;
		bool bCSV = FParse::Param(*Command, TEXT("CSV"));
		if (bCSV)
		{
			DumpFormat = FLowLevelMemTracker::EDumpFormat::CSV;
		}

		UE::LLM::ESizeParams SizeParams = UE::LLM::ESizeParams::Default;
		bool bSnapshot = FParse::Param(*Command, TEXT("SNAPSHOT"));
		if (bSnapshot)
		{
			EnumAddFlags(SizeParams, UE::LLM::ESizeParams::RelativeToSnapshot);
		}

		ELLMTagSet TagSet = ELLMTagSet::None;
		if (FParse::Param(*Command, TEXT("Assets")))
		{
			TagSet = ELLMTagSet::Assets;
		}
		else if (FParse::Param(*Command, TEXT("AssetClasses")))
		{
			TagSet = ELLMTagSet::AssetClasses;
		}
		else if (FParse::Param(*Command, TEXT("UObjectClasses")))
		{
			TagSet = ELLMTagSet::UObjectClasses;
		}
		else if (FParse::Param(*Command, TEXT("CodeOrContent")))
		{
			TagSet = ELLMTagSet::CodeOrContent;
		}

		FLowLevelMemTracker::Get().DumpToLog(DumpFormat, &Ar, SizeParams, TagSet);
	}));

#if !PLATFORM_HAS_MULTITHREADED_PREMAIN
bool FLowLevelMemTracker::TryEnterEnabled(UE::LLMPrivate::FEnableStateScopeLock& ScopeLock)
{
	return IsEnabled();
}
#else // !PLATFORM_HAS_MULTITHREADED_PREMAIN

namespace UE::LLMPrivate
{

FRWLock& GetEnableStateLock()
{
	static FRWLock Lock;
	return Lock;
}

} // namespace UE::LLMPrivate

bool FLowLevelMemTracker::TryEnterEnabled(UE::LLMPrivate::FEnableStateScopeLock& ScopeLock)
{
	using namespace UE::LLMPrivate;

	switch (EnabledState)
	{
	case EEnabled::NotYetKnown:
		ScopeLock.Inner.Emplace(UE::LLMPrivate::GetEnableStateLock());
		// Evaluate EnabledState again since it may have changed under ProcessCommandLine's WriteLock.
		return IsEnabled();
	case EEnabled::Disabled:
		return false;
	case EEnabled::Enabled:
		return true;
	default:
		checkNoEntry();
		return false;
	}
}
#endif // !PLATFORM_HAS_MULTITHREADED_PREMAIN

namespace UE::LLM
{

FString GetLLMProfilingDir()
{
	return FPaths::ProfilingDir() + TEXT("LLM/");
}

} // namespace UE::LLM

void FLowLevelMemTracker::DumpToLog(EDumpFormat DumpFormat, FOutputDevice* OutputDevice, UE::LLM::ESizeParams SizeParams, ELLMTagSet TagSet)
{
	if (!IsEnabled())
	{
		return;
	}
	GetGlobals()->DumpToLogInner(DumpFormat, OutputDevice, SizeParams, TagSet);
}

namespace UE::LLMPrivate
{

void FLLMGlobals::DumpToLogInner(EDumpFormat DumpFormat, FOutputDevice* OutputDevice, UE::LLM::ESizeParams SizeParams, ELLMTagSet TagSet)
{
	const float InvToMb = 1.0 / (1024 * 1024);
	FOutputDevice& Ar = *(OutputDevice ? OutputDevice : GLog);
	if (DumpFormat == EDumpFormat::CSV)
	{
		Ar.Logf(TEXT(",TagName,SizeMB,PeakMB,Tracker,PathName"));
	}
	else
	{
		Ar.Logf(TEXT("%40s %12s %12s  Tracker PathName"), TEXT("TagName"), TEXT("SizeMB"), TEXT("PeakMB"));
	}

	for (ELLMTracker TrackerType : {ELLMTracker::Default, ELLMTracker::Platform})
	{
		const TCHAR* TrackerName = TrackerType == ELLMTracker::Default ? TEXT("Default") : TEXT("Platform");

		struct FTagLine
		{
			FString TagName;
			FString Line;
			int64 Size;
		};

		UE::LLM::ESizeParams CurrentSizeParams = SizeParams;
		UE::LLM::ESizeParams PeakSizeParams = CurrentSizeParams;
		EnumAddFlags(PeakSizeParams, UE::LLM::ESizeParams::ReportPeak);

		TArray<FTagLine> TagLines;
		for (const UE::LLMPrivate::FTagData* TagData : GetTrackedTags(TrackerType, TagSet))
		{
			if (!TagData->IsReportable())
			{
				continue;
			}
			int64 CurrentAmount = GetTagAmountForTracker(TrackerType, TagData, SizeParams);
			int64 PeakAmount = GetTagAmountForTracker(TrackerType, TagData, PeakSizeParams);
			FString TagName = GetTagUniqueName(TagData).ToString();
			FString TagLine;
			if (DumpFormat == EDumpFormat::CSV)
			{
				TagLine = FString::Printf(TEXT(",%s,%.3f,%.3f,%s,%s"),
					*TagName,
					static_cast<float>(CurrentAmount) * InvToMb,
					static_cast<float>(PeakAmount) * InvToMb,
					TrackerName,
					*GetTagDisplayPathName(TagData));
			}
			else
			{
				TagLine = FString::Printf(TEXT("%40s %12.3f %12.3f %8s %s"),
					*TagName,
					static_cast<float>(CurrentAmount) * InvToMb,
					static_cast<float>(PeakAmount) * InvToMb,
					TrackerName,
					*GetTagDisplayPathName(TagData));
			}
			TagLines.Add(FTagLine{ MoveTemp(TagName), MoveTemp(TagLine), CurrentAmount });
		}
		TagLines.Sort([](const FTagLine& A, const FTagLine& B)
			{
				if (A.Size != B.Size)
				{
					return A.Size > B.Size;
				}
				return A.TagName < B.TagName;
			});
		for (FTagLine& TagLine : TagLines)
		{
			Ar.Logf(TEXT("%s"), *TagLine.Line);
		}
	}
}

} // namespace UE::LLMPrivate

DECLARE_LLM_MEMORY_STAT(TEXT("LLM Overhead"), STAT_LLMOverheadTotal, STATGROUP_LLMOverhead);

// LLM Summary stats referenced by ELLMTagNames
DEFINE_STAT(STAT_EngineSummaryLLM);
DEFINE_STAT(STAT_ProjectSummaryLLM);
DEFINE_STAT(STAT_NetworkingSummaryLLM);
DEFINE_STAT(STAT_AudioSummaryLLM);
DEFINE_STAT(STAT_TrackedTotalSummaryLLM);
DEFINE_STAT(STAT_MeshesSummaryLLM);
DEFINE_STAT(STAT_PhysicsSummaryLLM);
DEFINE_STAT(STAT_PhysXSummaryLLM);
DEFINE_STAT(STAT_ChaosSummaryLLM);
DEFINE_STAT(STAT_UObjectSummaryLLM);
DEFINE_STAT(STAT_AnimationSummaryLLM);
DEFINE_STAT(STAT_StaticMeshSummaryLLM);
DEFINE_STAT(STAT_MaterialsSummaryLLM);
DEFINE_STAT(STAT_ParticlesSummaryLLM);
DEFINE_STAT(STAT_NiagaraSummaryLLM);
DEFINE_STAT(STAT_UISummaryLLM);
DEFINE_STAT(STAT_NavigationSummaryLLM);
DEFINE_STAT(STAT_TexturesSummaryLLM);
DEFINE_STAT(STAT_MediaStreamingSummaryLLM);

// LLM stats referenced by ELLMTagNames
DECLARE_LLM_MEMORY_STAT(TEXT("Total"), STAT_TotalLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("Untracked"), STAT_UntrackedLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("Total"), STAT_PlatformTotalLLM, STATGROUP_LLMPlatform);
DECLARE_LLM_MEMORY_STAT(TEXT("Tracked Total"), STAT_TrackedTotalLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("Untagged"), STAT_UntaggedTotalLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("WorkingSetSize"), STAT_WorkingSetSizeLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("PagefileUsed"), STAT_PagefileUsedLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("Tracked Total"), STAT_PlatformTrackedTotalLLM, STATGROUP_LLMPlatform);
DECLARE_LLM_MEMORY_STAT(TEXT("Untagged"), STAT_PlatformUntaggedTotalLLM, STATGROUP_LLMPlatform);
DECLARE_LLM_MEMORY_STAT(TEXT("Untracked"), STAT_PlatformUntrackedLLM, STATGROUP_LLMPlatform);
DECLARE_LLM_MEMORY_STAT(TEXT("Overhead"), STAT_PlatformOverheadLLM, STATGROUP_LLMPlatform);
DECLARE_LLM_MEMORY_STAT(TEXT("OS Available"), STAT_PlatformOSAvailableLLM, STATGROUP_LLMPlatform);
DECLARE_LLM_MEMORY_STAT(TEXT("FMalloc"), STAT_FMallocLLM, STATGROUP_LLMPlatform);
DECLARE_LLM_MEMORY_STAT(TEXT("FMalloc Unused"), STAT_FMallocUnusedLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("RHI Unused"), STAT_RHIUnusedLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("ThreadStack"), STAT_ThreadStackLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("ThreadStackPlatform"), STAT_ThreadStackPlatformLLM, STATGROUP_LLMPlatform);
DECLARE_LLM_MEMORY_STAT(TEXT("Program Size"), STAT_ProgramSizePlatformLLM, STATGROUP_LLMPlatform);
DECLARE_LLM_MEMORY_STAT(TEXT("Program Size"), STAT_ProgramSizeLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("OOM Backup Pool"), STAT_OOMBackupPoolPlatformLLM, STATGROUP_LLMPlatform);
DECLARE_LLM_MEMORY_STAT(TEXT("OOM Backup Pool"), STAT_OOMBackupPoolLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("GenericPlatformMallocCrash"), STAT_GenericPlatformMallocCrashLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("GenericPlatformMallocCrash"), STAT_GenericPlatformMallocCrashPlatformLLM, STATGROUP_LLMPlatform);
DECLARE_LLM_MEMORY_STAT(TEXT("Engine Misc"), STAT_EngineMiscLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("TaskGraph Misc Tasks"), STAT_TaskGraphTasksMiscLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("Linear Allocator"), STAT_LinearAllocatorLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("Audio"), STAT_AudioLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("AudioMisc"), STAT_AudioMiscLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("AudioSoundWaves"), STAT_AudioSoundWavesLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("AudioSoundWaveProxies"), STAT_AudioSoundWaveProxiesLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("AudioMixer"), STAT_AudioMixerLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("AudioMixerPlugins"), STAT_AudioMixerPluginsLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("AudioPrecache"), STAT_AudioPrecacheLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("AudioDecompress"), STAT_AudioDecompressLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("AudioRealtimePrecache"), STAT_AudioRealtimePrecacheLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("AudioFullDecompress"), STAT_AudioFullDecompressLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("AudioStreamCache"), STAT_AudioStreamCacheLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("AudioStreamCacheCompressedData"), STAT_AudioStreamCacheCompressedDataLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("AudioSynthesis"), STAT_AudioSynthesisLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("RealTimeCommunications"), STAT_RealTimeCommunicationsLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("FName"), STAT_FNameLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("FNameCompressed"), STAT_FNameCompressedLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("Networking"), STAT_NetworkingLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("Meshes"), STAT_MeshesLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("Stats"), STAT_StatsLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("Shaders"), STAT_ShadersLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("PSO"), STAT_PSOLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("Textures"), STAT_TexturesLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("TextureMetaData"), STAT_TextureMetaDataLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("VirtualTextureSystem"), STAT_VirtualTextureSystemLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("Render Targets"), STAT_RenderTargetsLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("SceneRender"), STAT_SceneRenderLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("RHIMisc"), STAT_RHIMiscLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("AsyncLoading"), STAT_AsyncLoadingLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("UObject"), STAT_UObjectLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("Animation"), STAT_AnimationLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("StaticMesh"), STAT_StaticMeshLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("Materials"), STAT_MaterialsLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("Particles"), STAT_ParticlesLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("Niagara"), STAT_NiagaraLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("GPUSort"), STAT_GPUSortLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("GC"), STAT_GCLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("UI"), STAT_UILLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("NavigationRecast"), STAT_NavigationRecastLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("Physics"), STAT_PhysicsLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("PhysX"), STAT_PhysXLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("PhysXGeometry"), STAT_PhysXGeometryLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("PhysXLandscape"), STAT_PhysXLandscapeLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("PhysXTrimesh"), STAT_PhysXTrimeshLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("PhysXConvex"), STAT_PhysXConvexLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("PhysXAllocator"), STAT_PhysXAllocatorLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("Chaos"), STAT_ChaosLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("ChaosGeometry"), STAT_ChaosGeometryLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("ChaosAcceleration"), STAT_ChaosAccelerationLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("ChaosParticles"), STAT_ChaosParticlesLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("ChaosLandscape"), STAT_ChaosLandscapeLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("ChaosTrimesh"), STAT_ChaosTrimeshLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("ChaosConvex"), STAT_ChaosConvexLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("ChaosScene"), STAT_ChaosSceneLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("ChaosUpdate"), STAT_ChaosUpdateLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("ChaosActor"), STAT_ChaosActorLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("ChaosBody"), STAT_ChaosBodyLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("ChaosConstraint"), STAT_ChaosConstraintLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("ChaosMaterial"), STAT_ChaosMaterialLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("EnginePreInit"), STAT_EnginePreInitLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("EngineInit"), STAT_EngineInitLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("Rendering Thread"), STAT_RenderingThreadLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("LoadMap Misc"), STAT_LoadMapMiscLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("StreamingManager"), STAT_StreamingManagerLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("Graphics"), STAT_GraphicsPlatformLLM, STATGROUP_LLMPlatform);
DECLARE_LLM_MEMORY_STAT(TEXT("FileSystem"), STAT_FileSystemLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("Localization"), STAT_LocalizationLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("AssetRegistry"), STAT_AssetRegistryLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("ConfigSystem"), STAT_ConfigSystemLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("InitUObject"), STAT_InitUObjectLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("VideoRecording"), STAT_VideoRecordingLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("Replays"), STAT_ReplaysLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("CsvProfiler"), STAT_CsvProfilerLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("MaterialInstance"), STAT_MaterialInstanceLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("SkeletalMesh"), STAT_SkeletalMeshLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("InstancedMesh"), STAT_InstancedMeshLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("Landscape"), STAT_LandscapeLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("MediaStreaming"), STAT_MediaStreamingLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("ElectraPlayer"), STAT_ElectraPlayerLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("WMFPlayer"), STAT_WMFPlayerLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("MMIO"), STAT_PlatformMMIOLLM, STATGROUP_LLMPlatform);
DECLARE_LLM_MEMORY_STAT(TEXT("VirtualMemory"), STAT_PlatformVMLLM, STATGROUP_LLMPlatform);
DECLARE_LLM_MEMORY_STAT(TEXT("CustomName"), STAT_CustomName, STATGROUP_LLMFULL);

namespace UE::LLMPrivate
{

const TCHAR* ToString(ETagReferenceSource ReferenceSource);
void SetMemoryStatByFName(FName Name, int64 Amount);
void ValidateUniqueName(FStringView UniqueName);

typedef TArray<TPair<FLLMInitialisedCallback, UPTRINT>, TInlineAllocator<1>> FInitializedCallbacksArray;
FInitializedCallbacksArray& GetInitializedCallbacks()
{
	static FInitializedCallbacksArray Array;
	return Array;
}

typedef TArray<TPair<FTagCreationCallback, UPTRINT>, TInlineAllocator<1>> FTagCreationCallbacksArray;
FTagCreationCallbacksArray& GetTagCreationCallbacks()
{
	static FTagCreationCallbacksArray Array;
	return Array;
}

void FPrivateCallbacks::AddInitialisedCallback(FLLMInitialisedCallback Callback, UPTRINT UserData)
{
	FLowLevelMemTracker::Get().BootstrapInitialise();

	FInitializedCallbacksArray& Callbacks = UE::LLMPrivate::GetInitializedCallbacks();
	Callbacks.Add(TPair<FLLMInitialisedCallback, UPTRINT> { Callback, UserData });
}

void FPrivateCallbacks::AddTagCreationCallback(FTagCreationCallback Callback, UPTRINT UserData)
{
	FLowLevelMemTracker::Get().BootstrapInitialise();

	FTagCreationCallbacksArray& Callbacks = UE::LLMPrivate::GetTagCreationCallbacks();
	Callbacks.Add(TPair<FTagCreationCallback, UPTRINT> { Callback, UserData });
}

void FPrivateCallbacks::RemoveTagCreationCallback(FTagCreationCallback Callback)
{
	FLowLevelMemTracker::Get().BootstrapInitialise();

	FTagCreationCallbacksArray& Callbacks = UE::LLMPrivate::GetTagCreationCallbacks();
	Callbacks.RemoveAll([Callback](TPair<FTagCreationCallback, UPTRINT>& Pair) { return Pair.Key == Callback; });
}

} // namespace UE::LLMPrivate

static FName GetTagName_CustomName()
{
	// This FName can be read before c++ global constructors are complete, by LLM_SCOPE_BY_BOOTSTRAP_TAG
	// so it needs to be a function static rather than a global.
	static FName TagName_CustomName("CustomName");
	return TagName_CustomName;
}
static FName TagName_Untagged("Untagged");
static FName TagName_UntaggedAsset("UntaggedAsset");
static FName TagName_UntaggedAssetClass("UntaggedAssetClass");
static FName TagName_UntaggedUObjectClasses("UntaggedUObjectClasses");
LLM_DEFINE_TAG(CodeOrContent_Code, FName(TEXT("Code")), NAME_None, NAME_None, NAME_None, ELLMTagSet::CodeOrContent);
LLM_DEFINE_TAG(CodeOrContent_Content, FName(TEXT("Content")), NAME_None, NAME_None, NAME_None, ELLMTagSet::CodeOrContent);

FName LLMGetTagUniqueName(ELLMTag Tag)
{
#define LLM_TAG_NAME_ARRAY(Enum,Str,Stat,Group,ParentTag) FName(TEXT(#Enum)),
	static const FName UniqueNames[] = { LLM_ENUM_GENERIC_TAGS(LLM_TAG_NAME_ARRAY) };
#undef LLM_TAG_NAME_ARRAY

	int32 Index = static_cast<int32>(Tag);
	if (Index < 0)
	{
		return NAME_None;
	}
	else if (Index < UE_ARRAY_COUNT(UniqueNames))
	{
		return UniqueNames[Index];
	}
	else if (Index < LLM_CUSTOM_TAG_START)
	{
		return NAME_None;
	}
	else if (Index <= LLM_CUSTOM_TAG_END)
	{
		static FName CustomNames[LLM_CUSTOM_TAG_COUNT];
		static bool bCustomNamesInitialized = false;
		if (!bCustomNamesInitialized)
		{
			TStringBuilder<256> UniqueNameBuffer;

			for (int32 CreateIndex = LLM_CUSTOM_TAG_START; CreateIndex <= LLM_CUSTOM_TAG_END; ++CreateIndex)
			{
				UniqueNameBuffer.Reset();
				UniqueNameBuffer.Appendf(TEXT("ELLMTag%d"), CreateIndex);
				CustomNames[CreateIndex - LLM_CUSTOM_TAG_START] = FName(UniqueNameBuffer);
			}
			bCustomNamesInitialized = true;
		}
		return CustomNames[Index - LLM_CUSTOM_TAG_START];
	}
	else
	{
		return NAME_None;
	}
}

extern const TCHAR* LLMGetTagName(ELLMTag Tag)
{
#define LLM_TAG_NAME_ARRAY(Enum,Str,Stat,Group,ParentTag) TEXT(Str),
	static TCHAR const* Names[] = { LLM_ENUM_GENERIC_TAGS(LLM_TAG_NAME_ARRAY) };
#undef LLM_TAG_NAME_ARRAY

	int32 Index = static_cast<int32>(Tag);
	if (Index >= 0 && Index < UE_ARRAY_COUNT(Names))
	{
		return Names[Index];
	}
	else
	{
		return nullptr;
	}
}

extern const FName LLMGetUntaggedTagName(ELLMTagSet TagSet)
{
	switch (TagSet)
	{
		case ELLMTagSet::None:
		{
			return TagName_Untagged;
		}
		case ELLMTagSet::Assets:
		{
			return TagName_UntaggedAsset;
		}
		case ELLMTagSet::AssetClasses:
		{
			return TagName_UntaggedAssetClass;
		}
		case ELLMTagSet::UObjectClasses:
		{
			return TagName_UntaggedUObjectClasses;
		}
		case ELLMTagSet::CodeOrContent:
		{
			return LLMTagDeclaration_CodeOrContent_Code.GetUniqueName();
		}
		default:
		{
			return NAME_None;
		}
	}
}

extern const ANSICHAR* LLMGetTagNameANSI(ELLMTag Tag)
{
#define LLM_TAG_NAME_ARRAY(Enum,Str,Stat,Group,ParentTag) Str,
	static ANSICHAR const* Names[] = { LLM_ENUM_GENERIC_TAGS(LLM_TAG_NAME_ARRAY) };
#undef LLM_TAG_NAME_ARRAY

	int32 Index = static_cast<int32>(Tag);
	if (Index >= 0 && Index < UE_ARRAY_COUNT(Names))
	{
		return Names[Index];
	}
	else
	{
		return nullptr;
	}
}

FLowLevelMemTracker& FLowLevelMemTracker::Construct()
{
	static UE::LLMPrivate::FLLMGlobals Tracker;
	TrackerInstance = &Tracker;
	return Tracker;
}

FLowLevelMemTracker* FLowLevelMemTracker::TrackerInstance = nullptr;
UE::LLMPrivate::EEnabled FLowLevelMemTracker::EnabledState = UE::LLMPrivate::EEnabled::NotYetKnown;

static const TCHAR* InvalidLLMTagName = TEXT("?");

bool FLowLevelMemTracker::IsInitialized() const
{
	// IsEnabled is not checked here because Inner can be called even if not enabled.
	return GetGlobals()->IsInitializedInner();
}

bool FLowLevelMemTracker::IsConfigured() const
{
	// IsEnabled is not checked here because Inner can be called even if not enabled.
	return GetGlobals()->IsConfiguredInner();
}

void FLowLevelMemTracker::BootstrapInitialise()
{
	// IsEnabled is not checked here because there is some work to do in Inner function even if not enabled.
	GetGlobals()->BootstrapInitializeInner();
}

void FLowLevelMemTracker::OnPreFork()
{
	if (!IsEnabled())
	{
		return;
	}
	GetGlobals()->OnPreForkInner();
}

void FLowLevelMemTracker::UpdateStatsPerFrame(const TCHAR* LogName)
{
	// IsEnabled is not checked here because there is some work to do in Inner function even if not enabled.
	GetGlobals()->UpdateStatsPerFrameInner(LogName);
}

void FLowLevelMemTracker::Tick()
{
	if (!IsEnabled())
	{
		return;
	}
	GetGlobals()->TickInner();
}

void FLowLevelMemTracker::SetProgramSize(uint64 InProgramSize)
{
	if (!IsEnabled())
	{
		return;
	}
	GetGlobals()->SetProgramSizeInner(InProgramSize);
}

void FLowLevelMemTracker::ProcessCommandLine(const TCHAR* CmdLine)
{
	// IsEnabled is not checked here because there is some work to do in Inner function even if not enabled.
	GetGlobals()->ProcessCommandLineInner(CmdLine);
}

uint64 FLowLevelMemTracker::GetTotalTrackedMemory(ELLMTracker Tracker)
{
	if (!IsEnabled())
	{
		return 0;
	}
	return GetGlobals()->GetTotalTrackedMemoryInner(Tracker);
}

void FLowLevelMemTracker::OnLowLevelAlloc(ELLMTracker Tracker, const void* Ptr, uint64 Size, ELLMTag DefaultTag,
	ELLMAllocType AllocType, bool bTrackInMemPro)
{
	UE::LLMPrivate::FEnableStateScopeLock EnableScopeLock;
	if (!TryEnterEnabled(EnableScopeLock))
	{
		return;
	}
	GetGlobals()->OnLowLevelAllocInner(Tracker, Ptr, Size, DefaultTag, AllocType, bTrackInMemPro);
}

void FLowLevelMemTracker::OnLowLevelAlloc(ELLMTracker Tracker, const void* Ptr, uint64 Size, FName DefaultTag,
	ELLMAllocType AllocType, bool bTrackInMemPro)
{
	UE::LLMPrivate::FEnableStateScopeLock EnableScopeLock;
	if (!TryEnterEnabled(EnableScopeLock))
	{
		return;
	}
	GetGlobals()->OnLowLevelAllocInner(Tracker, Ptr, Size, DefaultTag, AllocType, bTrackInMemPro);
}

void FLowLevelMemTracker::OnLowLevelFree(ELLMTracker Tracker, const void* Ptr,
	ELLMAllocType AllocType, bool bTrackInMemPro)
{
	UE::LLMPrivate::FEnableStateScopeLock EnableScopeLock;
	if (!TryEnterEnabled(EnableScopeLock))
	{
		return;
	}
	GetGlobals()->OnLowLevelFreeInner(Tracker, Ptr, AllocType, bTrackInMemPro);
}

void FLowLevelMemTracker::OnLowLevelChangeInMemoryUse(ELLMTracker Tracker, int64 DeltaMemory, ELLMTag DefaultTag,
	ELLMAllocType AllocType)
{
	UE::LLMPrivate::FEnableStateScopeLock EnableScopeLock;
	if (!TryEnterEnabled(EnableScopeLock))
	{
		return;
	}
	GetGlobals()->OnLowLevelChangeInMemoryUseInner(Tracker, DeltaMemory, DefaultTag, AllocType);
}

void FLowLevelMemTracker::OnLowLevelChangeInMemoryUse(ELLMTracker Tracker, int64 DeltaMemory, FName DefaultTag,
	ELLMAllocType AllocType)
{
	UE::LLMPrivate::FEnableStateScopeLock EnableScopeLock;
	if (!TryEnterEnabled(EnableScopeLock))
	{
		return;
	}
	GetGlobals()->OnLowLevelChangeInMemoryUseInner(Tracker, DeltaMemory, DefaultTag, AllocType);
}

void FLowLevelMemTracker::OnLowLevelAllocMoved(ELLMTracker Tracker, const void* Dest, const void* Source,
	ELLMAllocType AllocType)
{
	UE::LLMPrivate::FEnableStateScopeLock EnableScopeLock;
	if (!TryEnterEnabled(EnableScopeLock))
	{
		return;
	}
	GetGlobals()->OnLowLevelAllocMovedInner(Tracker, Dest, Source, AllocType);
}

bool FLowLevelMemTracker::Exec(const TCHAR* Cmd, FOutputDevice& Ar)
{
	if (!IsEnabled())
	{
		if (FParse::Command(&Cmd, TEXT("LLMEM")))
		{
			UE_LOGF(LogHAL, Warning, "Cannot execute LLMEM command! LLM is not enabled.");
			return true;
		}
		return false;
	}
	return GetGlobals()->ExecInner(Cmd, Ar);
}

bool FLowLevelMemTracker::IsTagSetActive(ELLMTagSet Set)
{
	if (!IsEnabled())
	{
		return false;
	}
	UE::LLMPrivate::FLLMGlobals* Globals = GetGlobals();
	Globals->BootstrapInitialise();
	return Globals->IsTagSetActiveInner(Set);
}

bool FLowLevelMemTracker::ShouldReduceThreads()
{
#if LLM_ENABLED_REDUCE_THREADS
	if (!IsEnabled())
	{
		return false;
	}
	return GetGlobals()->ShouldReduceThreadsInner();
#else
	return false;
#endif
}

void FLowLevelMemTracker::RegisterPlatformTag(int32 Tag, const TCHAR* Name, FName StatName, FName SummaryStatName,
	int32 ParentTag)
{
	// IsEnabled is not checked here because there is some work to do in Inner function even if not enabled.
	GetGlobals()->RegisterPlatformTagInner(Tag, Name, StatName, SummaryStatName, ParentTag);
}

void FLowLevelMemTracker::RegisterProjectTag(int32 Tag, const TCHAR* Name, FName StatName, FName SummaryStatName,
	int32 ParentTag)
{
	// IsEnabled is not checked here because there is some work to do in Inner function even if not enabled.
	GetGlobals()->RegisterProjectTagInner(Tag, Name, StatName, SummaryStatName, ParentTag);
}

void GlobalRegisterTagDeclaration(FLLMTagDeclaration& TagDeclaration)
{
	if (!FLowLevelMemTracker::IsEnabled())
	{
		return;
	}
	UE::LLMPrivate::FLLMGlobals::GetInner().RegisterTagDeclaration(TagDeclaration);
}

void FLowLevelMemTracker::FinishInitialise()
{
	// IsEnabled is not checked here because there is some work to do in Inner function even if not enabled.
	GetGlobals()->FinishInitializeInner();
}

TArray<const UE::LLMPrivate::FTagData*> FLowLevelMemTracker::GetTrackedTags(ELLMTagSet TagSet)
{
	if (!IsEnabled())
	{
		return TArray<const UE::LLMPrivate::FTagData*>();
	}
	return GetGlobals()->GetTrackedTagsInner(TagSet);
}

TArray<const UE::LLMPrivate::FTagData*> FLowLevelMemTracker::GetTrackedTags(ELLMTracker Tracker, ELLMTagSet TagSet /* = ELLMTagSet::None */)
{
	if (!IsEnabled())
	{
		return TArray<const UE::LLMPrivate::FTagData*>();
	}
	return GetGlobals()->GetTrackedTagsInner(Tracker, TagSet);
}

void FLowLevelMemTracker::GetTrackedTagsNamesWithAmount(TMap<FName, uint64>& TagsNamesWithAmount, ELLMTracker Tracker, ELLMTagSet TagSet)
{
	if (!IsEnabled())
	{
		return;
	}
	GetGlobals()->GetTrackedTagsNamesWithAmountInner(TagsNamesWithAmount, Tracker, TagSet);
}

void FLowLevelMemTracker::GetTrackedTagsNamesWithAmountFiltered(TMap<FName, uint64>& TagsNamesWithAmount, ELLMTracker Tracker, ELLMTagSet TagSet, TArray<FLLMTagSetAllocationFilter>& Filters)
{
	if (!IsEnabled())
	{
		return;
	}
	GetGlobals()->GetTrackedTagsNamesWithAmountFilteredInner(TagsNamesWithAmount, Tracker, TagSet, Filters);
}

bool FLowLevelMemTracker::FindTagByName(const TCHAR* Name, uint64& OutTag, ELLMTagSet InTagSet /*= ELLMTagSet::None*/) const
{
	using namespace UE::LLMPrivate;

	if (!IsEnabled())
	{
		return false;
	}
	return GetGlobals()->FindTagByNameInner(Name, OutTag, InTagSet);
}

FName FLowLevelMemTracker::FindTagDisplayName(uint64 Tag) const
{
	if (!IsEnabled())
	{
		return NAME_None;
	}
	return GetGlobals()->FindTagDisplayNameInner(Tag);
}

FName FLowLevelMemTracker::FindPtrDisplayName(void* Ptr) const
{
	if (!IsEnabled())
	{
		return NAME_None;
	}
	return GetGlobals()->FindPtrDisplayNameInner(Ptr);
}

FName FLowLevelMemTracker::GetTagDisplayName(const UE::LLMPrivate::FTagData* TagData) const
{
	return GetGlobals()->GetTagDisplayNameInner(TagData);
}

FString FLowLevelMemTracker::GetTagDisplayPathName(const UE::LLMPrivate::FTagData* TagData) const
{
	// IsEnabled is not checked here because Inner is valid to call even if not enabled.
	return GetGlobals()->GetTagDisplayPathNameInner(TagData);
}

void FLowLevelMemTracker::GetTagDisplayPathName(const UE::LLMPrivate::FTagData* TagData,
	FStringBuilderBase& OutPathName, int32 MaxLen) const
{
	// IsEnabled is not checked here because Inner is valid to call even if not enabled.
	return GetGlobals()->GetTagDisplayPathNameInner(TagData, OutPathName, MaxLen);
}

FName FLowLevelMemTracker::GetTagUniqueName(const UE::LLMPrivate::FTagData* TagData) const
{
	// IsEnabled is not checked here because Inner is valid to call even if not enabled.
	return GetGlobals()->GetTagUniqueNameInner(TagData);
}

const UE::LLMPrivate::FTagData* FLowLevelMemTracker::GetTagParent(const UE::LLMPrivate::FTagData* TagData) const
{
	// IsEnabled is not checked here because Inner is valid to call even if not enabled.
	return GetGlobals()->GetTagParentInner(TagData);
}

bool FLowLevelMemTracker::GetTagIsEnumTag(const UE::LLMPrivate::FTagData* TagData) const
{
	// IsEnabled is not checked here because Inner is valid to call even if not enabled.
	return GetGlobals()->GetTagIsEnumTagInner(TagData);
}

ELLMTag FLowLevelMemTracker::GetTagClosestEnumTag(const UE::LLMPrivate::FTagData* TagData) const
{
	// IsEnabled is not checked here because Inner is valid to call even if not enabled.
	return GetGlobals()->GetTagClosestEnumTagInner(TagData);
}

int64 FLowLevelMemTracker::GetTagAmountForTracker(ELLMTracker Tracker, const UE::LLMPrivate::FTagData* TagData,
	UE::LLM::ESizeParams SizeParams)
{
	if (!IsEnabled())
	{
		return 0;
	}
	return GetGlobals()->GetTagAmountForTrackerInner(Tracker, TagData, SizeParams);
}

int64 FLowLevelMemTracker::GetTagAmountForTracker(ELLMTracker Tracker, FName Tag, ELLMTagSet TagSet,
	UE::LLM::ESizeParams SizeParams)
{
	if (!IsEnabled())
	{
		return 0;
	}
	return GetGlobals()->GetTagAmountForTrackerInner(Tracker, Tag, TagSet, SizeParams);
}

int64 FLowLevelMemTracker::GetTagAmountForTracker(ELLMTracker Tracker, ELLMTag Tag, UE::LLM::ESizeParams SizeParams)
{
	if (!IsEnabled())
	{
		return 0;
	}
	return GetGlobals()->GetTagAmountForTrackerInner(Tracker, Tag, SizeParams);
}

void FLowLevelMemTracker::SetTagAmountForTracker(ELLMTracker Tracker, ELLMTag Tag, int64 Amount, bool bAddToTotal)
{
	if (!IsEnabled())
	{
		return;
	}
	GetGlobals()->SetTagAmountForTrackerInner(Tracker, Tag, Amount, bAddToTotal);
}

void FLowLevelMemTracker::SetTagAmountForTracker(ELLMTracker Tracker, FName Tag, ELLMTagSet TagSet, int64 Amount, bool bAddToTotal)
{
	if (!IsEnabled())
	{
		return;
	}
	GetGlobals()->SetTagAmountForTrackerInner(Tracker, Tag, TagSet, Amount, bAddToTotal);
}

const UE::LLMPrivate::FTagData* FLowLevelMemTracker::GetActiveTagData(ELLMTracker Tracker, ELLMTagSet TagSet /*= ELLMTagSet::None*/)
{
	if (!IsEnabled())
	{
		return nullptr;
	}
	return GetGlobals()->GetActiveTagDataInner(Tracker, TagSet);
}

uint64 FLowLevelMemTracker::DumpTag(ELLMTracker Tracker, const char* FileName, int LineNumber)
{
	if (!IsEnabled())
	{
		return static_cast<int64>(ELLMTag::Untagged);
	}
	return GetGlobals()->DumpTagInner(Tracker, FileName, LineNumber);
}

void FLowLevelMemTracker::PublishDataSingleFrame()
{
	// IsEnabled is not checked here because Inner is valid to call even if not enabled.
	GetGlobals()->PublishDataSingleFrameInner();
}

namespace UE::LLMPrivate
{

FLLMGlobals::FLLMGlobals()
	: TagDatas(nullptr)
	, TagDataNameMap(nullptr)
	, TagDataEnumMap(nullptr)
	, TraceTagFilter(nullptr)
	, ProgramSize(0)
	, MemoryUsageCurrentOverhead(0)
	, MemoryUsagePlatformTotalUntracked(0)
	, bFirstTimeUpdating(true)
	, bCanEnable(true)
	, bCsvWriterEnabled(false)
	, bTraceWriterEnabled(false)
	, bInitializedTracking(false)
	, bIsBootstrapping(false)
	, bFullyInitialized(false)
	, bConfigurationComplete(false)
	, bTagAdded(false)
	, bAutoPublish(true)
	, bPublishSingleFrame(false)
	, bCapturedSizeSnapshot(false)
{
	// set the alloc functions
	LLMAllocFunction PlatformLLMAlloc = NULL;
	LLMFreeFunction PlatformLLMFree = NULL;
	int32 Alignment = 0;
	if (!FPlatformMemory::GetLLMAllocFunctions(PlatformLLMAlloc, PlatformLLMFree, Alignment))
	{
		EnabledState = EEnabled::Disabled;
		bCanEnable = false;
		bConfigurationComplete = true;
		return;
	}
	LLMCheck(FMath::IsPowerOfTwo(Alignment));

	Allocator.Initialize(PlatformLLMAlloc, PlatformLLMFree, Alignment);
	FLLMAllocator::Get() = &Allocator;

	// only None is on by default
	for (int32 Index = 0; Index < static_cast<int32>(ELLMTagSet::Max); Index++)
	{
		ActiveSets[Index] = Index == static_cast<int32>(ELLMTagSet::None);
	}
}

FLLMGlobals::~FLLMGlobals()
{
	EnabledState = EEnabled::Disabled;
	Clear();
	FLLMAllocator::Get() = nullptr;
}

void FLLMGlobals::BootstrapInitializeInner()
{
	if (bInitializedTracking)
	{
		return;
	}
	UE::TUniqueLock BootstrapLock(BootstrapMutex);
	if (bInitializedTracking)
	{
		return;
	}
	// just make sure this part isn't reentrant.
	if (bBeganInitializedTracking != false)
	{
		UE_LOGF(LogInit, Fatal, "FLowLevelMemTracker::BootstrapInitialize is not reentrant");
	}
	bBeganInitializedTracking = true;

	for (int32 TrackerIndex = 0; TrackerIndex < static_cast<int32>(ELLMTracker::Max); ++TrackerIndex)
	{
		FLLMTracker* Tracker = Allocator.New<FLLMTracker>(*this);

		Trackers[TrackerIndex] = Tracker;

		Tracker->Initialize(static_cast<ELLMTracker>(TrackerIndex), &Allocator);
	}

	BootstrapTagDatas();
	BootstrapAllocationGroups();
	static_assert((uint8)ELLMTracker::Max == 2,
		"You added a tracker, without updating FLowLevelMemTracker::BootstrapInitialize (and probably need to update macros)");
	GetTracker(ELLMTracker::Platform)->SetTotalTags(FindOrAddTagData(ELLMTag::PlatformUntaggedTotal),
		FindOrAddTagData(ELLMTag::PlatformTrackedTotal));
	GetTracker(ELLMTracker::Default)->SetTotalTags(FindOrAddTagData(ELLMTag::UntaggedTotal),
		FindOrAddTagData(ELLMTag::TrackedTotal));


	// calculate program size early on... the platform can call SetProgramSize later if it sees fit
	InitializeProgramSize();

	FPlatformMisc::MemoryBarrier();

	bInitializedTracking = true;
}

void FLLMGlobals::Clear()
{
	if (!bInitializedTracking)
	{
		return;
	}

	LLMCheck(!IsEnabled()); // tracking must be stopped at this point or it will crash while tracking its own destruction
	for (int32 TrackerIndex = 0; TrackerIndex < static_cast<int32>(ELLMTracker::Max); TrackerIndex++)
	{
		GetTracker((ELLMTracker)TrackerIndex)->Clear();
		Allocator.Delete(Trackers[TrackerIndex]);
		Trackers[TrackerIndex] = nullptr;
	}

	if (TraceTagFilter)
	{
		Allocator.Delete(TraceTagFilter);
		TraceTagFilter = nullptr;
	}

	ClearAllocationGroups();
	ClearTagDatas();
	Allocator.Clear();
	bFullyInitialized = false;
	bInitializedTracking = false;
}

void FLLMGlobals::OnPreForkInner()
{
	FLLMTracker& DefaultTracker = *GetTracker(ELLMTracker::Default);
	FLLMTracker& PlatformTracker = *GetTracker(ELLMTracker::Platform);

	DefaultTracker.OnPreFork();
	PlatformTracker.OnPreFork();
}

void FLLMGlobals::UpdateStatsPerFrameInner(const TCHAR* LogName)
{
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(LLM);
#if UE_ENABLE_ARRAY_SLACK_TRACKING
	// Slack tracking, when compiled in, can run even when regular LLM tracking is disabled
	LlmTrackArrayTick();
#endif

	if (!IsEnabled())
	{
		if (bFirstTimeUpdating)
		{
			// UpdateStatsPerFrame is usually called from the game thread, but can sometimes be called from the
			// async loading thread, so enter a lock for it
			UE::TUniqueLock UpdateLock(UpdateMutex);
			if (bFirstTimeUpdating)
			{
				// Write the saved overhead value to the stats system; this allows us to see the overhead that is
				// always there even when disabled (unless the #define completely removes support, of course)
				bFirstTimeUpdating = false;
				// Don't call Update; Trackers no longer exist. But do publish the recorded values.
				PublishDataPerFrame(LogName);
			}
		}
		return;
	}

	// UpdateStatsPerFrame is usually called from the game thread, but can sometimes be called from the
	// async loading thread, so enter a lock for it
	UE::TUniqueLock UpdateLock(UpdateMutex);
	BootstrapInitialise();

	if (bFirstTimeUpdating)
	{
		// Nothing needed here yet
		UE_LOGF(LogInit, Log, "First time updating LLM stats...");
		bFirstTimeUpdating = false;
	}
	TickInnerNoLock();

	if (bAutoPublish || bPublishSingleFrame)
	{
		PublishDataPerFrame(LogName);
		bPublishSingleFrame = false;
	}
}

void FLLMGlobals::TickInner()
{
	// TickInnerNoLock is usually called from the game thread, but can sometimes be called from the async loading thread,
	// so enter a lock for it.
	UE::TUniqueLock UpdateLock(UpdateMutex);
	TickInnerNoLock();
}

void FLLMGlobals::TickInnerNoLock()
{
	if (bFullyInitialized)
	{
		// We call tick when not fully initialised to get the overhead when disabled. When not initialised, we have to
		// avoid the portion of the tick that uses tags.

		// Get the platform to update any custom tags; this must be done before the aggregation that occurs in
		// GetTracker()->Update.
		FPlatformMemory::UpdateCustomLLMTags();

		FCoreDelegates::OnLowLevelMemTrackerTick.Broadcast();

		UpdateTags();

		// update the trackers
		for (int32 TrackerIndex = 0; TrackerIndex < static_cast<int32>(ELLMTracker::Max); TrackerIndex++)
		{
			GetTracker((ELLMTracker)TrackerIndex)->Update();
		}
	}
	FLLMTracker& DefaultTracker = *Trackers[static_cast<int32>(ELLMTracker::Default)];
	FLLMTracker& PlatformTracker = *Trackers[static_cast<int32>(ELLMTracker::Platform)];

	const int64 TrackedTotal = DefaultTracker.GetTrackedTotal();

	// Cache the amount of memory used early, since some of these functions (FindOrAddTagData) can
	// cause allocations, which will throw the numbers off slightly.
	FPlatformMemoryStats PlatformStats = FPlatformMemory::GetStats();
#if PLATFORM_DESKTOP
	// virtual is working set + paged out memory.
	const int64 PlatformProcessMemory = static_cast<int64>(PlatformStats.UsedVirtual);
#elif PLATFORM_ANDROID
	// On some Android devices total used mem (VmRss + VmSwap) does not include GPU memory allocations
	// and so it's going to be lower than TrackedTotal
	const int64 PlatformProcessMemory = FMath::Max((int64)PlatformStats.UsedPhysical, TrackedTotal);

	PlatformTracker.SetTagAmountInUpdate(FindOrAddTagData(ELLMTag::PlatformMMIO), (int64)PlatformStats.RssFile, false);
#else
	const int64 PlatformProcessMemory = static_cast<int64>(PlatformStats.UsedPhysical);
#endif
	// Update how much overhead LLM is using. Note we used to also added sizeof(FLowLevelMemTracker), but this
	// isn't allocated and is in the data segment. So adding it throws the numbers off.
	MemoryUsageCurrentOverhead = Allocator.GetTotal();
	PlatformTracker.SetTagAmountInUpdate(FindOrAddTagData(ELLMTag::PlatformOverhead), MemoryUsageCurrentOverhead, true);

	// Calculate FMalloc unused stat and set it in the Default tracker.
	const int64 FMallocAmount = DefaultTracker.GetAllocTypeAmount(ELLMAllocType::FMalloc);
	const int64 FMallocPlatformAmount = PlatformTracker.GetTagAmount(FindOrAddTagData(ELLMTag::FMalloc));
	const int64 CachedFreeMemory = UE::Private::GMalloc ? UE::Private::GMalloc->GetTotalFreeCachedMemorySize() : 0;
	int64 FMallocUnused = FMallocPlatformAmount - FMallocAmount - CachedFreeMemory;
	if (FMallocPlatformAmount == 0)
	{
		// We do not have instrumentation for this allocator, and so can not calculate how much memory it is using
		// internally. Set unused to 0 for this case.
		FMallocUnused = 0;
	}
	DefaultTracker.SetTagAmountInUpdate(FindOrAddTagData(ELLMTag::FMallocUnused), FMallocUnused, true);

	// Determine the UnusedRHI amount by finding the difference between the Platform and Default tracker.
	int64 PlatformRHIAmount = PlatformTracker.GetAllocTypeAmount(ELLMAllocType::RHI);
	int64 DefaultRHIAmount = DefaultTracker.GetAllocTypeAmount(ELLMAllocType::RHI);
	int64 UnusedRHIAmount = PlatformRHIAmount - DefaultRHIAmount;
	DefaultTracker.SetTagAmountInUpdate(FindOrAddTagData(ELLMTag::RHIUnused), UnusedRHIAmount, true);

	// Compare memory the platform thinks we have allocated to what we have tracked, including the program memory
	const int64 PlatformTrackedTotal = PlatformTracker.GetTrackedTotal();
	MemoryUsagePlatformTotalUntracked = FMath::Max<int64>(0, PlatformProcessMemory - PlatformTrackedTotal);

	PlatformTracker.SetTagAmountInUpdate(FindOrAddTagData(ELLMTag::PlatformTotal), PlatformProcessMemory, false);
	PlatformTracker.SetTagAmountInUpdate(FindOrAddTagData(ELLMTag::PlatformUntracked), MemoryUsagePlatformTotalUntracked, false);
	PlatformTracker.SetTagAmountInUpdate(FindOrAddTagData(ELLMTag::PlatformOSAvailable), PlatformStats.AvailablePhysical, false);

	// remove the MemoryUsageCurrentOverhead from the "Total" for the default LLM as it's not something anyone needs
	// to investigate when finding what to reduce the platform LLM will have the info
	const int64 DefaultProcessMemory = PlatformProcessMemory - MemoryUsageCurrentOverhead;
	const int64 ImmediatelyFreeableCachedMemory = UE::Private::GMalloc ? UE::Private::GMalloc->GetImmediatelyFreeableCachedMemorySize() : 0;
	// we need to subtract diff between total cached free mem and immediately freeable cached memory as it is not accounted anywhere in the LLM and would artificially inflate Untracked.
	// while any new allocations from that cached, but not immediately freeable memory would be properly accounted by the LLM
	const int64 DefaultUntracked = FMath::Max<int64>(0, DefaultProcessMemory - TrackedTotal - CachedFreeMemory + ImmediatelyFreeableCachedMemory);
	DefaultTracker.SetTagAmountInUpdate(FindOrAddTagData(ELLMTag::Total), DefaultProcessMemory, false);
	DefaultTracker.SetTagAmountInUpdate(FindOrAddTagData(ELLMTag::Untracked), DefaultUntracked, false);

#if PLATFORM_WINDOWS
	DefaultTracker.SetTagAmountInUpdate(FindOrAddTagData(ELLMTag::WorkingSetSize), PlatformStats.UsedPhysical, false);
	DefaultTracker.SetTagAmountInUpdate(FindOrAddTagData(ELLMTag::PagefileUsed), PlatformStats.UsedVirtual, false);
#endif
}

void FLLMGlobals::UpdateTags()
{
	if (!bTagAdded)
	{
		return;
	}

	bTagAdded = false;
	bool bNeedsResort = false;
	{
		UE::TSharedLock Lock(TagDataMutex);
		// Cannot use a ranged-for because FinishConstruct can drop our lock and add elements to TagDatas
		// Check the valid-index condition on every loop iteration
		for (int32 TagIndex = 0; TagIndex < TagDatas->Num(); ++TagIndex)
		{
			FTagData* TagData = (*TagDatas)[TagIndex];
			FinishConstruct(TagData, ETagReferenceSource::FunctionAPI);
			const FTagData* Parent = TagData->GetParent();
			if (Parent && Parent->GetIndex() > TagData->GetIndex())
			{
				bNeedsResort = true;
			}
		}
	}
	if (bNeedsResort)
	{
		// Prevent threads from reading their AllocationGroups or Tags while we are mutating tags.
		for (int32 TrackerIndex = 0; TrackerIndex < static_cast<int32>(ELLMTracker::Max); TrackerIndex++)
		{
			GetTracker(static_cast<ELLMTracker>(TrackerIndex))->LockAllThreadsSharedData(true);
		}

		FTagDataArray* OldTagDatas;
		UE::TUniqueLock Lock(TagDataMutex);
		SortTags(OldTagDatas);

		for (int32 TrackerIndex = 0; TrackerIndex < static_cast<int32>(ELLMTracker::Max); TrackerIndex++)
		{
			GetTracker(static_cast<ELLMTracker>(TrackerIndex))->OnTagsResorted(*OldTagDatas);
			GetTracker(static_cast<ELLMTracker>(TrackerIndex))->LockAllThreadsSharedData(false);
		}

		Allocator.Delete(OldTagDatas);
	}
}

void FLLMGlobals::SortTags(FTagDataArray*& OutOldTagDatas)
{
	// Caller is responsible for holding a UniqueLock on TagDataMutex
	OutOldTagDatas = TagDatas;
	TagDatas = Allocator.New<FTagDataArray>();
	FTagDataArray& LocalTagDatas = *TagDatas;
	LocalTagDatas.Reserve(OutOldTagDatas->Num());
	for (FTagData* TagData : *OutOldTagDatas)
	{
		LocalTagDatas.Add(TagData);
	}

	auto GetEdges = [&LocalTagDatas](int32 Vertex, int* Edges, int& NumEdges)
	{
		NumEdges = 0;
		const FTagData* Parent = LocalTagDatas[Vertex]->GetParent();
		if (Parent)
		{
			Edges[NumEdges++] = Parent->GetIndex();
		}
	};

	LLMAlgo::TopologicalSortLeafToRoot(LocalTagDatas, GetEdges);

	// Set each tag's index to its new position in the sort order
	for (int32 n = 0; n < LocalTagDatas.Num(); ++n)
	{
		LocalTagDatas[n]->SetIndex(n);
	}
}

void FLLMGlobals::PublishDataPerFrame(const TCHAR* LogName)
{
	// set overhead stats
	SET_MEMORY_STAT(STAT_LLMOverheadTotal, MemoryUsageCurrentOverhead);
	if (IsEnabled())
	{
		FLLMTracker& DefaultTracker = *GetTracker(ELLMTracker::Default);
		FLLMTracker& PlatformTracker = *GetTracker(ELLMTracker::Platform);

		bool bTrackPeaks = CVarLLMTrackPeaks.GetValueOnAnyThread() != 0;

		UE::LLM::ESizeParams SizeParams(UE::LLM::ESizeParams::Default);
		if (bTrackPeaks)
		{
			EnumAddFlags(SizeParams, UE::LLM::ESizeParams::ReportPeak);
		}

		if (bCapturedSizeSnapshot)
		{
			EnumAddFlags(SizeParams, UE::LLM::ESizeParams::RelativeToSnapshot);
		}

#if !LLM_ENABLED_TRACK_PEAK_MEMORY
		if (bTrackPeaks)
		{
			static bool bWarningGiven = false;
			if (!bWarningGiven)
			{
				UE_LOGF(LogHAL, Warning,
					"Attempted to enable LLM.TrackPeaks, but LLM_ENABLED_TRACK_PEAK_MEMORY is not defined to 1. You will need to enable the define");
				bWarningGiven = true;
			}
		}
#endif

		DefaultTracker.PublishStats(SizeParams);
		PlatformTracker.PublishStats(SizeParams);

		if (bCsvWriterEnabled)
		{
			DefaultTracker.PublishCsv(SizeParams);
			PlatformTracker.PublishCsv(SizeParams);
		}

		if (bTraceWriterEnabled)
		{
			DefaultTracker.PublishTrace(SizeParams);
			PlatformTracker.PublishTrace(SizeParams);
		}

#if LLM_CSV_PROFILER_WRITER_ENABLED
		if (FCsvProfiler::Get()->IsCapturing())
		{
			DefaultTracker.PublishCsvProfiler(SizeParams);
			PlatformTracker.PublishCsvProfiler(SizeParams);
		}
#endif
	}

	if (LogName != nullptr)
	{
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("---> Untracked memory at %s = %.2f mb\n"),
			LogName, (double)MemoryUsagePlatformTotalUntracked / (1024.0 * 1024.0));
	}
}

void FLLMGlobals::InitializeProgramSize()
{
	// On Android we report ProgramSize via AndroidMemoryProbeTick
	// and on iOS we report ProgramSize via FAppleMemoryProbe::ScanMemory
#if !PLATFORM_ANDROID && !PLATFORM_IOS
	if (!ProgramSize)
	{
		FPlatformMemoryStats Stats = FPlatformMemory::GetStats();
		ProgramSize = Stats.TotalPhysical - Stats.AvailablePhysical;

		GetTracker(ELLMTracker::Platform)->TrackMemory(ELLMTag::ProgramSizePlatform, ProgramSize, ELLMAllocType::System);
		GetTracker(ELLMTracker::Default)->TrackMemory(ELLMTag::ProgramSize, ProgramSize, ELLMAllocType::System);
	}
#endif
}

void FLLMGlobals::SetProgramSizeInner(uint64 InProgramSize)
{
	if (!IsEnabled())
	{
		return;
	}
	BootstrapInitialise();

	int64 ProgramSizeDiff = static_cast<int64>(InProgramSize) - ProgramSize;

	ProgramSize = static_cast<int64>(InProgramSize);

	GetTracker(ELLMTracker::Platform)->TrackMemory(ELLMTag::ProgramSizePlatform, ProgramSizeDiff, ELLMAllocType::System);
	GetTracker(ELLMTracker::Default)->TrackMemory(ELLMTag::ProgramSize, ProgramSizeDiff, ELLMAllocType::System);
}

void FLLMGlobals::ProcessCommandLineInner(const TCHAR* CmdLine)
{
	CSV_METADATA(TEXT("LLM"), TEXT("0"));

#if LLM_AUTO_ENABLE
	// LLM is always on, regardless of command line
	bool bShouldDisable = false;
#elif LLM_COMMANDLINE_ENABLES_FUNCTIONALITY
	// if we require commandline to enable it, then we are disabled if it's not there
	bool bShouldDisable = FParse::Param(CmdLine, TEXT("LLM")) == false;
#else
	// if we allow commandline to disable us, then we are disabled if it's there
	bool bShouldDisable = FParse::Param(CmdLine, TEXT("NOLLM")) == true;
#endif

	bool bLocalCsvWriterEnabled = FParse::Param(CmdLine, TEXT("LLMCSV"));
	bool bLocalTraceWriterEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(MemTagChannel);
	// automatically enable LLM if only csv or trace output is active
	if (bLocalCsvWriterEnabled || bLocalTraceWriterEnabled)
	{
		if (bLocalTraceWriterEnabled)
		{
			UE_LOGF(LogInit, Log, "LLM enabled due to UE_TRACE MemTagChannel being enabled");
		}
		bShouldDisable = false;
	}

	bAutoPublish = FParse::Param(CmdLine, TEXT("LLMDISABLEAUTOPUBLISH")) == false;

	if (!bCanEnable)
	{
		LLMCheck(EnabledState == EEnabled::Disabled);
		if (!bShouldDisable)
		{
			UE_LOGF(LogInit, Log,
				"LLM - Ignoring request to enable LLM; it is not available on the current platform");
		}
		return;
	}

	ON_SCOPE_EXIT
	{
		bConfigurationComplete = true;
	};

	if (bShouldDisable)
	{
		// Before we shutdown, update once so we can publish the overhead-when-disabled later during the first
		// call to UpdateStatsPerFrame.
		if (IsEnabled())
		{
			Tick();
		}

		{
#if PLATFORM_HAS_MULTITHREADED_PREMAIN
			// The EnableStateLock must be limited in scope because other code in the function
			// allocates memory and would block on the readlock. Trying to take it around the write of the
			// EnabledState is sufficient; this will cause us to wait until threads already in OnLowLevelAlloc exit the
			// function and clear their readlock, and will cause new calls to those functions to block while we're waiting
			// and call IsEnabled again after we release the lock.
			FWriteScopeLock EnableStateLock(GetEnableStateLock());
#endif

			EnabledState = EEnabled::Disabled;
		}
		bCsvWriterEnabled = false;
		bTraceWriterEnabled = false;
		bCanEnable = false; // Reenabling after a clear is not implemented
		Clear();
		return;
	}
	CSV_METADATA(TEXT("LLM"), TEXT("1"));

	// activate tag sets (we ignore None set, it's always on)
	FString SetList;
	bool LocalActiveSets[static_cast<uint8>(ELLMTagSet::Max)] = { false };
	LocalActiveSets[static_cast<uint8>(ELLMTagSet::None)] = true;

	static_assert(static_cast<uint8>(ELLMTagSet::Max) == 5, "You added a tagset without updating FLowLevelMemTracker::ProcessCommandLine");
	if (FParse::Value(CmdLine, TEXT("LLMTAGSETS="), SetList, false /* bShouldStopOnSeparator */))
	{
#if LLM_ALLOW_NAMES_TAGS
		TArray<FString> Sets;
		SetList.ParseIntoArray(Sets, TEXT(","), true);
		for (FString& Set : Sets)
		{
			if (Set == TEXT("Assets"))
			{
				LocalActiveSets[static_cast<int32>(ELLMTagSet::Assets)] = true;
			}
			else if (Set == TEXT("AssetClasses"))
			{
				LocalActiveSets[static_cast<int32>(ELLMTagSet::AssetClasses)] = true;
			}
			else if (Set == TEXT("UObjectClasses"))
			{
				LocalActiveSets[static_cast<int32>(ELLMTagSet::UObjectClasses)] = true;
			}
			else if (Set == TEXT("CodeOrContent"))
			{
				LocalActiveSets[static_cast<int32>(ELLMTagSet::CodeOrContent)] = true;
			}
		}
#else
		// We do not support activating ELLMTagSets other than ELLMTagSet::None when not using LLM_ALLOW_NAMES_TAGS,
		// because we need to assume we can use just the first TagData of the FActiveTags to
		// map from ELLMTag -> FActiveTags, see FindOrAddAllocationGroup.
		UE_LOGF(LogInit, Warning,
			"Attempted to use LLMTAGSETS, but LLM_ALLOW_NAMES_TAGS is not defined to 1. You will need to enable the define");
#endif
	}
	// These calls to FindOrAddTagData must occur outside of EnableStateLock to avoid deadlock due to
	// recursive entry.
	const FTagData* DefaultTags[static_cast<uint8>(ELLMTagSet::Max)] = { nullptr };
	for (uint8 TagSetIndex = 0; TagSetIndex < static_cast<uint8>(ELLMTagSet::Max); ++TagSetIndex)
	{
		if (!LocalActiveSets[TagSetIndex])
		{
			continue;
		}
		DefaultTags[TagSetIndex] = FindOrAddDefaultTagData(static_cast<ELLMTagSet>(TagSetIndex));
	}

	{
#if PLATFORM_HAS_MULTITHREADED_PREMAIN
		// The EnableStateLock must be limited in scope because other code in the function
		// allocates memory and would block on the readlock. Trying to take it around the write of the
		// EnabledState and ActiveSets is sufficient; this will cause us to wait until threads already in
		// OnLowLevelAlloc exit the function and clear their readlock, and will cause new calls to
		// those functions to block while we're modifying IsEnabled and ActiveSets
		// and call IsEnabled again after we release the lock.
		FWriteScopeLock EnableStateLock(GetEnableStateLock());
#endif
		EnabledState = EEnabled::Enabled;
		FMemory::Memcpy(ActiveSets, LocalActiveSets, sizeof(ActiveSets));
		OnActiveSetsInitialized(DefaultTags);
	}

	bCsvWriterEnabled = bLocalCsvWriterEnabled;
	bTraceWriterEnabled = bLocalTraceWriterEnabled;
	FinishInitialise();

	if (FParse::Value(CmdLine, TEXT("LLMTraceTagFilter="), SetList, false /* bShouldStopOnSeparator */))
	{
		LLMCheck(TraceTagFilter == nullptr); // ProcessCommandLine() is expected to be called only once
		TArray<FString> TagNames;
		SetList.ParseIntoArray(TagNames, TEXT(","), true);
		if (!TagNames.IsEmpty())
		{
			if (!TraceTagFilter)
			{
				TraceTagFilter = Allocator.New<FTagFilter>();
			}
			for (FString& TagName : TagNames)
			{
				TraceTagFilter->Add(FName(TagName));
			}
			// Update bIsTraceReportable flag for already created tags.
			// TagData is supposed to be immutable once constructed, but we use bIsTraceReportable flag
			// as a mutable member to quickly filter tags for the trace purposes.
			UE::TSharedLock Lock(TagDataMutex);
			for (FTagData* TagData : (*TagDatas))
			{
				if (!TraceTagFilter->Contains(TagData->GetName()))
				{
					TagData->SetIsTraceReportable(false);
				}
			}
		}
	}

	// Commandline overrides for console variables
	int TrackPeaks = 0;
	if (FParse::Value(CmdLine, TEXT("LLMTrackPeaks="), TrackPeaks))
	{
		CVarLLMTrackPeaks->Set(TrackPeaks);
	}

	UE_LOGF(LogInit, Log, "LLM enabled CsvWriter: %ls TraceWriter: %ls",
		bCsvWriterEnabled ? TEXT("on") : TEXT("off"), bTraceWriterEnabled ? TEXT("on") : TEXT("off"));

	// Disable hitchdetector because LLM is already slow enough.
	if (!FParse::Param(CmdLine, TEXT("DetectHitchesWithLLM")))
	{
		// Schedule disabling when it's safe to init HitchHeartBeat
		FCoreDelegates::GetOnPostEngineInit().AddLambda([]{
			// Calling Get() will instantiate FGameThreadHitchHeartBeat if it is not already.
			FGameThreadHitchHeartBeat::Get().Stop();
		});
	}
}

void FLLMGlobals::OnActiveSetsInitialized(const FTagData* DefaultTags[static_cast<uint8>(ELLMTagSet::Max)])
{
	// Called within write scope of EnableStateLock, #if PLATFORM_HAS_MULTITHREADED_PREMAIN
	// Otherwise, called only before multithreading begins

	// Satisfy our contract for thread synchronization of tags (even if not technically necessary because there
	// are no threads currently): prevent threads from reading AllocationGroups or Tags while we are mutating
	// AllocationGroups.
	UE::TUniqueLock ScopeLockAllocationGroups(AllocationGroupsMutex);
	for (int32 TrackerIndex = 0; TrackerIndex < static_cast<int32>(ELLMTracker::Max); TrackerIndex++)
	{
		GetTracker(static_cast<ELLMTracker>(TrackerIndex))->LockAllThreadsSharedData(true);
	}

	int32 NumActiveSets = 0;
	for (bool IsActive : ActiveSets)
	{
		if (IsActive)
		{
			++NumActiveSets;
		}
	}
	LLMCheck(NumActiveSets > 0);

	if (NumActiveSets != FActiveTags::GetGlobalNum())
	{
		// Before we are allowed to call FActiveTags::SetGlobalNum, we have to remove all FActiveTags
		// from the keys of TSets, because TSet keys are not allowed to mutate their TypeHash, and calling SetGlobalNum
		// will mutate the TypeHash of every FActiveTags.
		ActiveTagsToAllocationGroup->Empty();

		TArray<TArray<FAllocationGroupTrackingData, FDefaultLLMAllocator>, FDefaultLLMAllocator>
			ThreadStateAllocationGroupTrackingDatas[static_cast<int32>(ELLMTracker::Max)];
		for (int32 TrackerIndex = 0; TrackerIndex < static_cast<int32>(ELLMTracker::Max); TrackerIndex++)
		{
			FLLMTracker* Tracker = GetTracker(static_cast<ELLMTracker>(TrackerIndex));
			Tracker->UpdateThreads();
			Tracker->BeginActiveSetsChange(ThreadStateAllocationGroupTrackingDatas[TrackerIndex]);
		}

		FActiveTags::SetGlobalNum(NumActiveSets);
		FActiveTags NewDefaults;
		int32 IndexInActiveTags = 0;
		for (uint8 TagSetIndex = 0; TagSetIndex < static_cast<uint8>(ELLMTagSet::Max); ++TagSetIndex)
		{
			if (ActiveSets[TagSetIndex])
			{
				NewDefaults[IndexInActiveTags] = DefaultTags[TagSetIndex];
				++IndexInActiveTags;
			}
		}

		for (FAllocationGroup* AllocationGroup : (*AllocationGroups))
		{
			if (AllocationGroup->IsInitialized())
			{
				AllocationGroup->ConvertToPostCommandlineBootstrap(NewDefaults);
				ActiveTagsToAllocationGroup->Add(AllocationGroup->GetActiveTags(), AllocationGroup);
			}
		}

		for (int32 TrackerIndex = 0; TrackerIndex < static_cast<int32>(ELLMTracker::Max); TrackerIndex++)
		{
			GetTracker(static_cast<ELLMTracker>(TrackerIndex))->EndActiveSetsChange(NewDefaults,
				ThreadStateAllocationGroupTrackingDatas[TrackerIndex]);
		}
	}

	for (int32 TrackerIndex = 0; TrackerIndex < static_cast<int32>(ELLMTracker::Max); TrackerIndex++)
	{
		GetTracker(static_cast<ELLMTracker>(TrackerIndex))->LockAllThreadsSharedData(false);
	}
}

uint64 FLLMGlobals::GetTotalTrackedMemoryInner(ELLMTracker Tracker)
{
	BootstrapInitialise();

	return static_cast<uint64>(GetTracker(Tracker)->GetTrackedTotal());
}

void FLLMGlobals::OnLowLevelAllocInner(ELLMTracker Tracker, const void* Ptr, uint64 Size, ELLMTag DefaultTag,
	ELLMAllocType AllocType, bool bTrackInMemPro)
{
	BootstrapInitialise();

	GetTracker(Tracker)->TrackAllocation(Ptr, static_cast<int64>(Size), DefaultTag, AllocType, bTrackInMemPro);
}

void FLLMGlobals::OnLowLevelAllocInner(ELLMTracker Tracker, const void* Ptr, uint64 Size, FName DefaultTag,
	ELLMAllocType AllocType, bool bTrackInMemPro)
{
	BootstrapInitialise();

	GetTracker(Tracker)->TrackAllocation(Ptr, static_cast<int64>(Size), DefaultTag, AllocType, bTrackInMemPro);
}

void FLLMGlobals::OnLowLevelFreeInner(ELLMTracker Tracker, const void* Ptr,
	ELLMAllocType AllocType, bool bTrackInMemPro)
{
	BootstrapInitialise();

	if (Ptr != nullptr)
	{
		GetTracker(Tracker)->TrackFree(Ptr, AllocType, bTrackInMemPro);
	}
}

void FLLMGlobals::OnLowLevelChangeInMemoryUseInner(ELLMTracker Tracker, int64 DeltaMemory, ELLMTag DefaultTag,
	ELLMAllocType AllocType)
{
	BootstrapInitialise();
	GetTracker(Tracker)->TrackMemoryOfActiveTag(DeltaMemory, DefaultTag, AllocType);
}

void FLLMGlobals::OnLowLevelChangeInMemoryUseInner(ELLMTracker Tracker, int64 DeltaMemory, FName DefaultTag,
	ELLMAllocType AllocType)
{
	BootstrapInitialise();
	GetTracker(Tracker)->TrackMemoryOfActiveTag(DeltaMemory, DefaultTag, AllocType);
}

void FLLMGlobals::OnLowLevelAllocMovedInner(ELLMTracker Tracker, const void* Dest, const void* Source,
	ELLMAllocType AllocType)
{
	BootstrapInitialise();

	//update the allocation map
	GetTracker(Tracker)->OnAllocMoved(Dest, Source, AllocType);
}

FLLMTracker* FLLMGlobals::GetTracker(ELLMTracker Tracker)
{
	return Trackers[static_cast<int32>(Tracker)];
}

const FLLMTracker* FLLMGlobals::GetTracker(ELLMTracker Tracker) const
{
	return Trackers[static_cast<int32>(Tracker)];
}

bool FLLMGlobals::ExecInner(const TCHAR* Cmd, FOutputDevice& Ar)
{
	if (FParse::Command(&Cmd, TEXT("LLMEM")))
	{
		BootstrapInitialise();
		if (FParse::Command(&Cmd, TEXT("SPAMALLOC")))
		{
			int32 NumAllocs = 128;
			int64 MaxSize = FCString::Atoi(Cmd);
			if (MaxSize == 0)
			{
				MaxSize = 128 * 1024;
			}

			UpdateStatsPerFrame(TEXT("Before spam"));
			FPlatformMisc::LowLevelOutputDebugStringf(TEXT("----> Spamming %d allocations, from %d..%d bytes\n"),
				NumAllocs, MaxSize/2, MaxSize);

			TArray<void*> Spam;
			Spam.Reserve(NumAllocs);
			SIZE_T TotalSize = 0;
			for (int32 Index = 0; Index < NumAllocs; Index++)
			{
				SIZE_T Size = (FPlatformMath::Rand() % MaxSize / 2) + MaxSize / 2;
				TotalSize += Size;
				Spam.Add(FMemory::Malloc(Size));
			}
			FPlatformMisc::LowLevelOutputDebugStringf(TEXT("----> Allocated %d total bytes\n"), TotalSize);

			UpdateStatsPerFrame(TEXT("After spam"));

			for (int32 Index = 0; Index < Spam.Num(); Index++)
			{
				FMemory::Free(Spam[Index]);
			}

			UpdateStatsPerFrame(TEXT("After cleanup"));
		}
		else if (FParse::Command(&Cmd, TEXT("DUMPPRIVATESHARED")))
		{
			for (int32 TrackerIndex = 0; TrackerIndex < static_cast<int32>(ELLMTracker::Max); TrackerIndex++)
			{
				if (GetTracker((ELLMTracker)TrackerIndex)->DumpForkedAllocationInfo() == false)
				{
					FPlatformMisc::LowLevelOutputDebugString(TEXT("Failed to dumping forked allocation info (check platform supports it?)\n"));
					return true;
				}
			}
		}
		else if (FParse::Command(&Cmd, TEXT("REMEMBER")))
		{
			CSV_EVENT_GLOBAL(TEXT("LLM_REMEMBER"));

			UE::TUniqueLock UpdateLock(UpdateMutex);
			// Go through all the tags and REMEMBER the current recorded size values
			for (int32 TrackerIndex = 0; TrackerIndex < static_cast<int32>(ELLMTracker::Max); TrackerIndex++)
			{
				GetTracker((ELLMTracker)TrackerIndex)->CaptureTagSnapshot();
			}
			bCapturedSizeSnapshot = true;
		}
		else if (FParse::Command(&Cmd, TEXT("FORGET")))
		{
			CSV_EVENT_GLOBAL(TEXT("LLM_FORGET"));

			UE::TUniqueLock UpdateLock(UpdateMutex);
			// Go through all tags and make them forget their current size values
			for (int32 TrackerIndex = 0; TrackerIndex < static_cast<int32>(ELLMTracker::Max); TrackerIndex++)
			{
				GetTracker((ELLMTracker)TrackerIndex)->ClearTagSnapshot();
			}
			bCapturedSizeSnapshot = false;
		}
		else if (FParse::Command(&Cmd, TEXT("DUMPTAGS")))
		{
			if (GetTracker(ELLMTracker::Default)->DumpTags() == false)
			{
				FPlatformMisc::LowLevelOutputDebugString(TEXT("Failed to dump tags info\n"));
				return true;
			}
		}
		return true;
	}

	return false;
}

bool FLLMGlobals::IsTagSetActiveInner(ELLMTagSet Set)
{
	return IsTagSetScopeActiveInner(Set);
}

bool FLLMGlobals::IsTagSetScopeActiveInner(ELLMTagSet Set)
{
	// ELLMTagSets::AssetClasses and ELLMTagSets::CodeOrContent share scopes
	if (Set == ELLMTagSet::AssetClasses || Set == ELLMTagSet::CodeOrContent)
	{
		return ActiveSets[static_cast<int32>(ELLMTagSet::AssetClasses)]
			|| ActiveSets[static_cast<int32>(ELLMTagSet::CodeOrContent)];
	}
	else
	{
		return ActiveSets[static_cast<int32>(Set)];
	}
}

bool FLLMGlobals::IsTagSetRecordingActiveInner(ELLMTagSet Set)
{
	return ActiveSets[static_cast<int32>(Set)];
}

bool FLLMGlobals::ShouldReduceThreadsInner()
{
#if LLM_ENABLED_REDUCE_THREADS
	BootstrapInitialise();
	LLMCheckf(bConfigurationComplete,
		TEXT("ShouldReduceThreads has been called too early, before we processed the configuration settings required for it."));

	return IsTagRecordingActiveInner(ELLMTagSet::Assets) || IsTagRecordingActiveInner(ELLMTagSet::AssetClasses);
#else
	return false;
#endif
}

void FLLMGlobals::RegisterCustomTag(int32 Tag, ELLMTagSet TagSet, const TCHAR* InDisplayName, FName StatName,
	FName SummaryStatName, int32 ParentTag)
{
	LLMCheckf(Tag >= LLM_CUSTOM_TAG_START && Tag <= LLM_CUSTOM_TAG_END, TEXT("Tag %d out of range"), Tag);
	LLMCheckf(InDisplayName != nullptr, TEXT("Tag %d has no name"), Tag);
	LLMCheckf(ParentTag == -1 || ParentTag < LLM_TAG_COUNT, TEXT("Parent tag %d out of range"), ParentTag);

	FName DisplayName = InDisplayName ? InDisplayName : InvalidLLMTagName;
	ELLMTag EnumTag = static_cast<ELLMTag>(Tag);
	FName ParentName = ParentTag >= 0 ? LLMGetTagUniqueName(static_cast<ELLMTag>(ParentTag)) : NAME_None;

	RegisterTagData(LLMGetTagUniqueName(EnumTag), DisplayName, ParentName, StatName, SummaryStatName, true,
		EnumTag, false, ETagReferenceSource::CustomEnumTag, TagSet);
}

void FLLMGlobals::RegisterPlatformTagInner(int32 Tag, const TCHAR* Name, FName StatName, FName SummaryStatName,
	int32 ParentTag)
{
	MemoryTrace_AnnounceCustomTag(Tag, ParentTag, Name);

	// IsEnabled is checked in this Inner function because there is some work to do even if not enabled.
	if (!IsEnabled())
	{
		return;
	}
	BootstrapInitialise();

	LLMCheck(Tag >= static_cast<int32>(ELLMTag::PlatformTagStart) &&
		Tag <= static_cast<int32>(ELLMTag::PlatformTagEnd));
	RegisterCustomTag(Tag, ELLMTagSet::None, Name, StatName, SummaryStatName, ParentTag);
}

void FLLMGlobals::RegisterProjectTagInner(int32 Tag, const TCHAR* Name, FName StatName, FName SummaryStatName,
	int32 ParentTag)
{
	MemoryTrace_AnnounceCustomTag(Tag, ParentTag, Name);

	// IsEnabled is checked in this Inner function because there is some work to do even if not enabled.
	if (!IsEnabled())
	{
		return;
	}
	BootstrapInitialise();

	LLMCheck(Tag >= static_cast<int32>(ELLMTag::ProjectTagStart) && Tag <= static_cast<int32>(ELLMTag::ProjectTagEnd));
	RegisterCustomTag(Tag, ELLMTagSet::None, Name, StatName, SummaryStatName, ParentTag);
}

void FLLMGlobals::BootstrapTagDatas()
{
	// While bootstrapping we are not allowed to construct any FNames because the FName system may not yet have been
	// constructed. Construct not-fully-initialized TagDatas for the central list of ELLMTags.
	{
		UE::TUniqueLock Lock(TagDataMutex);
		bIsBootstrapping = true;

		TagDatas = Allocator.New<FTagDataArray>();
		TagDataNameMap = Allocator.New<FTagDataNameMap>();
		TagDataEnumMap = reinterpret_cast<FTagData**>(Allocator.Alloc(sizeof(FTagData*) * LLM_TAG_COUNT));
		FMemory::Memset(TagDataEnumMap, 0, sizeof(FTagData*) * LLM_TAG_COUNT);

#define REGISTER_ELLMTAG(Enum,Str,Stat,Group,ParentTag) \
		{ \
			ELLMTag EnumTag = ELLMTag::Enum; \
			int32 Index = static_cast<int32>(EnumTag); \
			LLMCheck(0 <= Index && Index < LLM_TAG_COUNT); \
			FTagData* TagData = Allocator.New<FTagData>(EnumTag); \
			TagData->SetIndex(TagDatas->Num()); \
			TagDatas->Add(TagData); \
			LLMCheck(TagDataEnumMap[Index] == nullptr); \
			TagDataEnumMap[Index] = TagData; \
		}
		LLM_ENUM_GENERIC_TAGS(REGISTER_ELLMTAG);
#undef REGISTER_ELLMTAG

#if LLM_ALLOW_NAMES_TAGS
		// The CustomName tag is an adapter for connecting LLM_SCOPE_BYNAME tags with platforms that used the
		// ELLMTag-based reporting. We want to hide this procedural tag in systems that use FName-based reporting;
		// if it is displayed it confusingly just displays a sum of every LLM_SCOPE_BYNAME tag.
		TagDataEnumMap[static_cast<int32>(ELLMTag::CustomName)]->SetIsReportable(false);
#endif
	}
}

void FLLMGlobals::BootstrapAllocationGroups()
{
	AllocationGroups = Allocator.New<FAllocationGroupArray>();
	AllocationGroupsFreeList = Allocator.New<FAllocationGroupArray>();
	AllocationGroupsUnreferencedSet = Allocator.New<FAllocationGroupSet>();
	ActiveTagsToAllocationGroup = Allocator.New<FMapActiveTagsToAllocationGroup>();
}

void FLLMGlobals::FinishInitializeInner()
{
	if (bFullyInitialized)
	{
		return;
	}
	BootstrapInitialise();

#if UE_ENABLE_ARRAY_SLACK_TRACKING
	GTrackArrayDetailedLock = new FCriticalSection();
#endif
	bFullyInitialized = true;
	// Make sure that FNames and Malloc have already been initialised, since we will use them during InitializeTagDatas
	// We force this by calling LLMGetTagUniqueName, which initializes FNames internally, and will therein trigger
	// FName system construction, which will itself trigger Malloc construction.
	(void)LLMGetTagUniqueName(ELLMTag::Untagged);
	InitializeTagDatas();

	FInitializedCallbacksArray& Callbacks = GetInitializedCallbacks();
	for (const TPair<FLLMInitialisedCallback, UPTRINT>& Callback : Callbacks)
	{
		Callback.Key(Callback.Value);
	}
	Callbacks.Empty();
}

void FLLMGlobals::InitializeTagDatas_SetLLMTagNames()
{
	TStringBuilder<256> NameBuffer;
	// Load all the names for the central list of ELLMTags (recording the allocations the FName system makes for the
	// construction of the names).
#define SET_ELLMTAG_NAMES(Enum,Str,Stat,Group,ParentTag) \
	{ \
		ELLMTag EnumTag = ELLMTag::Enum; \
		int32 Index = static_cast<int32>(EnumTag); \
		FTagData* TagData = TagDataEnumMap[Index]; \
		FName TagName = LLMGetTagUniqueName(EnumTag); \
		TagName.ToString(NameBuffer); \
		ValidateUniqueName(NameBuffer); \
		TagData->SetName(LLMGetTagUniqueName(EnumTag)); \
		TagData->SetDisplayName(Str); \
		TagData->SetStatName(Stat); \
		TagData->SetSummaryStatName(Group); \
		TagData->SetParentName(static_cast<int32>(ParentTag) == -1 ? \
			NAME_None : LLMGetTagUniqueName(static_cast<ELLMTag>(ParentTag))); \
	}
	LLM_ENUM_GENERIC_TAGS(SET_ELLMTAG_NAMES);
#undef SET_ELLMTAG_NAMES
}

void FLLMGlobals::InitializeTagDatas_FinishRegister()
{
	// Record the central list of ELLMTags in TagDataNameMap, and mark that bootstrapping is complete
	{
		UE::TUniqueLock Lock(TagDataMutex);

#define FINISH_REGISTER(Enum,Str,Stat,Group,ParentTag) \
		{ \
			ELLMTag EnumTag = ELLMTag::Enum; \
			int32 Index = static_cast<int32>(EnumTag); \
			FTagData* TagData = TagDataEnumMap[Index]; \
			FTagData*& ExistingTagData = TagDataNameMap->FindOrAdd(FTagDataNameKey(TagData->GetName(), TagData->GetTagSet()), nullptr); \
			if (ExistingTagData != nullptr) \
			{ \
				ReportDuplicateTagName(ExistingTagData, ETagReferenceSource::EnumTag); \
			} \
			ExistingTagData = TagData; \
		}
		LLM_ENUM_GENERIC_TAGS(FINISH_REGISTER);
#undef FINISH_REGISTER

		bIsBootstrapping = false;
	}
}

void FLLMGlobals::InitializeTagDatas()
{
	InitializeTagDatas_SetLLMTagNames();

	InitializeTagDatas_FinishRegister();

	// Construct the remaining startup tags; allocations when constructing these tags are known to consist only of the
	// central list of ELLMTags so we do not need to bootstrap.
	{
		// Construct LLM_DECLARE_TAGs
		FLLMTagDeclaration* List = FLLMTagDeclaration::GetList();
		while (List)
		{
			RegisterTagDeclaration(*List);
			List = List->Next;
		}
		FLLMTagDeclaration::AddCreationCallback(GlobalRegisterTagDeclaration);
	}

	// now let the platform add any custom tags
	FPlatformMemory::RegisterCustomLLMTags();

	// All parents in the ELLMTags and the initial modules' list of LLM_DEFINE_TAG must be contained within that same
	// set, so we can FinishConstruct them now, which we do in UpdateTags.
	bTagAdded = true;
	UpdateTags();
}

void FLLMGlobals::ClearTagDatas()
{
	UE::TUniqueLock Lock(TagDataMutex);
	FLLMTagDeclaration::ClearCreationCallbacks();

	Allocator.Free(TagDataEnumMap, sizeof(FTagData*) * LLM_TAG_COUNT);
	TagDataEnumMap = nullptr;
	Allocator.Delete(TagDataNameMap);
	TagDataNameMap = nullptr;
	for (FTagData* TagData : *TagDatas)
	{
		Allocator.Delete(TagData);
	}
	Allocator.Delete(TagDatas);
	TagDatas = nullptr;
}

void FLLMGlobals::ClearAllocationGroups()
{
	Allocator.Delete(ActiveTagsToAllocationGroup);
	ActiveTagsToAllocationGroup = nullptr;
	Allocator.Delete(AllocationGroupsFreeList);
	AllocationGroupsFreeList = nullptr;
	Allocator.Delete(AllocationGroupsUnreferencedSet);
	AllocationGroupsUnreferencedSet = nullptr;
	for (FAllocationGroup* AllocationGroup: *AllocationGroups)
	{
		Allocator.Delete(AllocationGroup);
	}
	Allocator.Delete(AllocationGroups);
	AllocationGroups = nullptr;
}

void FLLMGlobals::RegisterTagDeclaration(FLLMTagDeclaration& TagDeclaration)
{
	TagDeclaration.ConstructUniqueName();
	RegisterTagData(TagDeclaration.UniqueName, TagDeclaration.DisplayName, TagDeclaration.ParentTagName,
		TagDeclaration.StatName, TagDeclaration.SummaryStatName, false, ELLMTag::CustomName, false,
		ETagReferenceSource::Declare, TagDeclaration.TagSet);
}

FTagData& FLLMGlobals::RegisterTagData(FName Name, FName DisplayName, FName ParentName,
	FName StatName, FName SummaryStatName, bool bHasEnumTag, ELLMTag EnumTag, bool bIsStatTag,
	ETagReferenceSource ReferenceSource, ELLMTagSet TagSet/* = ELLMTagSet::None*/)
{
	LLMCheckf(!bIsBootstrapping,
		TEXT("A tag outside of LLM_ENUM_GENERIC_TAGS was requested from LLM_SCOPE or allocation while bootstrapping the names for LLM_ENUM_GENERIC_TAGS, this is not supported."));

	// If this allocates, that is okay. Set it to something small-as-possible-to-avoid-normally-allocating to prevent
	// adding a lot of stack space in the calling LLM_SCOPE code.
	TStringBuilder<256> NameBuffer;
	Name.ToString(NameBuffer);

	if (bHasEnumTag)
	{
		ValidateUniqueName(NameBuffer);
		// EnumTags can specify DisplayName (if they are central or if CustomTag registration provided it);
		// if not, they set DisplayName = UniqueName.
		// Enum tags only specify ParentName explicitly; if no ParentName is provided, they have no parent.
		if (DisplayName.IsNone())
		{
			DisplayName = Name;
		}
	}
	else if (TagSet != ELLMTagSet::None)
	{
		// Tags not in the None TagSet do not have parents and do not require name validation
		if (DisplayName.IsNone())
		{
			DisplayName = Name;
		}
	}
	else if (bIsStatTag)
	{
		// We set LLM UniqueName = <TheEntireString> and LLM DisplayName = StatDisplayName.
		// Stat tags do not specify their parent, and their parent is set to the CustomName aggregator.
		LLMCheck(ParentName.IsNone());
		if (DisplayName.IsNone())
		{
			DisplayName = Name;
		}
		ParentName = GetTagName_CustomName();
	}
	else
	{
		ValidateUniqueName(NameBuffer);
		// Normal defined-by-name tags supply unique names of the form Grandparent/.../Parent/Name.
		// ParentName and  DisplayName can be provided.

		// If both ParentName and /Parent/ are supplied, it is an error if they do not match.
		// All custom name tags have to be children of an ELLMTag. If no parent is set, it defaults to the proxy
		// parent CustomName.
		const TCHAR* LeafStart = NameBuffer.ToString();
		while (true)
		{
			const TCHAR* NextDivider = FCString::Strstr(LeafStart, TEXT("/"));
			if (!NextDivider)
			{
				break;
			}
			LeafStart = NextDivider + 1;
		}
		LLMCheckf(LeafStart[0] != '\0', TEXT("Invalid LLM custom name tag '%s'. Tag names must not end with /."),
			NameBuffer.ToString());
		if (LeafStart != NameBuffer.ToString())
		{
			FName ParsedParentName = FName(FStringView(NameBuffer.ToString(),
				UE_PTRDIFF_TO_INT32(LeafStart - 1 - NameBuffer.ToString())));
			if (!ParentName.IsNone() && ParentName != ParsedParentName)
			{
				TStringBuilder<128> ParentBuffer;
				ParentName.ToString(ParentBuffer);
				LLMCheckf(false,
					TEXT("Invalid LLM tag: parent supplied in tag declaration is '%s', which does not match the parent parsed from the tag unique name '%s'"),
					ParentBuffer.ToString(), NameBuffer.ToString());
			}
			ParentName = ParsedParentName;
		}
		else if (ParentName.IsNone())
		{
			ParentName = GetTagName_CustomName();
		}

		// Display name is set to the leaf portion of the unique name, and is overridden if DisplayName is provided.
		if (DisplayName.IsNone())
		{
			DisplayName = FName(LeafStart);
		}
	}

	UE::TUniqueLock Lock(TagDataMutex);
	FTagData*& TagDataForName = TagDataNameMap->FindOrAdd(FTagDataNameKey(Name, TagSet), nullptr);
	if (TagDataForName)
	{
		// The tag already exists; this can happen because two formal registrations have tried to add the same name,
		// but it can also happen due to a race condition when auto-adding a tag from LLM_SCOPE, if the tag is added
		// by another thread in between FindOrAddTagData's check of TagDataNameMap->Find and now. Note that it is not
		// valid for an LLM_SCOPE to be called before a formal registration (e.g. LLM_DECLARE_TAG). If a formal
		// registration exists for a tag, it must precede its use in any LLM_SCOPE calls.
		if (ParentName != TagDataForName->GetParentNameSafeBeforeFinishConstruct() ||
			StatName != TagDataForName->GetStatName() ||
			SummaryStatName != TagDataForName->GetSummaryStatName() ||
			bHasEnumTag != TagDataForName->HasEnumTag() ||
			(bHasEnumTag && EnumTag != TagDataForName->GetEnumTag()))
		{
			if (ReferenceSource != ETagReferenceSource::Scope &&
				ReferenceSource != ETagReferenceSource::ImplicitParent &&
				ReferenceSource != ETagReferenceSource::FunctionAPI)
			{
				ReportDuplicateTagName(TagDataForName, ReferenceSource);
			}
		}

		// Abandon the string work we've done and return the version that was added by the other source.
		return *TagDataForName;
	}

	FTagData* ParentData = nullptr;
	if (!ParentName.IsNone())
	{
		FTagData** ParentPtr = TagDataNameMap->Find(FTagDataNameKey(ParentName, TagSet));
		if (ParentPtr)
		{
			ParentData = *ParentPtr;
		}
	}

	FTagData* TagData;
	if (ParentName.IsNone() || ParentData)
	{
		TagData = Allocator.New<FTagData>(Name, TagSet, DisplayName, ParentData, StatName, SummaryStatName, bHasEnumTag,
			EnumTag, ReferenceSource);
	}
	else
	{
		TagData = Allocator.New<FTagData>(Name, TagSet, DisplayName, ParentName, StatName, SummaryStatName, bHasEnumTag,
			EnumTag, ReferenceSource);
	}
	TagData->SetIndex(TagDatas->Num());
	TagDatas->Add(TagData);

	TagDataForName = TagData;

	if (bHasEnumTag)
	{
		int32 Index = static_cast<int32>(EnumTag);
		LLMCheck(0 <= Index && Index < LLM_TAG_COUNT);
		FTagData*& TagDataForEnum = TagDataEnumMap[Index];
		if (TagDataForEnum != nullptr)
		{
			LLMCheckf(false, TEXT("LLM Error: Duplicate copies of enumtag %d"), Index);
		}
		TagDataForEnum = TagData;
	}

	bTagAdded = true;
	return *TagData;
}

void FLLMGlobals::ReportDuplicateTagName(FTagData* TagData, ETagReferenceSource ReferenceSource)
{
	// We're inside the LLM lock, so do not allow these GetName calls to allocate memory from FMalloc.
	if (ReferenceSource == ETagReferenceSource::FunctionAPI || ReferenceSource == ETagReferenceSource::Scope ||
		ReferenceSource == ETagReferenceSource::ImplicitParent)
	{
		LLMCheckf(false,
			TEXT("LLM Error: Unexpected call to RegisterTagData(%s) from LLM_SCOPE or function call when the tag already exists."),
			*WriteToString<FName::StringBufferSize>(TagData->GetName()));
	}
	else if (TagData->GetReferenceSource() == ETagReferenceSource::FunctionAPI ||
		TagData->GetReferenceSource() == ETagReferenceSource::Scope)
	{
		LLMCheckf(false, TEXT("LLM Error: Tag %s parsed from %s after it was already used in an LLM_SCOPE or LLM api call."),
			*WriteToString<FName::StringBufferSize>(TagData->GetName()), ToString(ReferenceSource));
	}
	else if (TagData->GetReferenceSource() == ETagReferenceSource::ImplicitParent)
	{
		LLMCheckf(false,
			TEXT("LLM Error: Tag %s parsed from %s after it was already used as an implicit parent in another tag. ")
			TEXT("Add LLM_DEFINE_TAG(% s) in cpp, or move it to the same module as the child tag using it, so that it will be defined before the child tag tries to use it."),
			*WriteToString<FName::StringBufferSize>(TagData->GetName()), ToString(ReferenceSource),
			*WriteToString<FName::StringBufferSize>(TagData->GetName()));
	}
	else
	{
		LLMCheckf(false, TEXT("LLM Error: Multiple occurrences of a unique tag name %s in ELLMTag or LLM_DEFINE_TAG. ")
			TEXT("First occurrence : % s.Second occurrence : % s."),
			*WriteToString<FName::StringBufferSize>(TagData->GetName()), ToString(TagData->GetReferenceSource()),
			ToString(ReferenceSource));
	}
}

void FLLMGlobals::FinishConstruct(FTagData* TagData, ETagReferenceSource ReferenceSource)
{
	// Caller is responsible for holding a SharedLock (NOT a UniqueLock) on TagDataMutex

	LLMCheck(TagData);
	if (TagData->IsFinishConstructed())
	{
		return;
	}
	if (bIsBootstrapping)
	{
		// Can't access Names yet; run the FinishConstruct later
		return;
	}

	if (!TagData->IsParentConstructed())
	{
		FName ParentName = TagData->GetParentName();
		if (ParentName.IsNone())
		{
			TagData->SetParent(nullptr);
		}
		else
		{
			FTagData** ParentDataPtr = TagDataNameMap->Find(FTagDataNameKey(ParentName, TagData->GetTagSet()));
			if (!ParentDataPtr)
			{
				// We have to drop the SharedLock to call RegisterTagData, which takes a UniqueLock.
				TagDataMutex.UnlockShared();
				FTagData* ParentTagData = &RegisterTagData(ParentName, NAME_None, NAME_None, NAME_None, NAME_None,
					false, ELLMTag::CustomName,false, ETagReferenceSource::ImplicitParent, TagData->GetTagSet());
				TagDataMutex.LockShared();
				if (TagData->IsFinishConstructed())
				{
					// Another thread got in and finished construction while we were outside of the lock.
					return;
				}
				ParentDataPtr = TagDataNameMap->Find(FTagDataNameKey(ParentName, TagData->GetTagSet()));
				LLMCheck(ParentDataPtr);
			}
			FTagData* ParentData = *ParentDataPtr;
			TagData->SetParent(ParentData);
		}
	}

	FTagData* ParentData = const_cast<FTagData*>(TagData->GetParent());
	if (ParentData)
	{
		// Make sure the parent chain is FinishConstructed as well, since GetContainingEnum or GetDisplayPath will be
		// called and walk up the parent chain.
		FinishConstruct(ParentData, ReferenceSource);
	}

	if (TraceTagFilter && !TraceTagFilter->Contains(TagData->GetName()))
	{
		TagData->SetIsTraceReportable(false);
	}

	TagData->SetFinishConstructed();

	// Broadcast the tag creation, except for generic tags which are constructed before any subscriber could
	// possibly have registered. Subscribers must instead read those from GetTrackedTags.
	if (!TagData->HasEnumTag() || TagData->GetEnumTag() >= ELLMTag::GenericTagCount)
	{
		// Leave the shared mutex while calling the callback, since the callback could be arbitrary
		// code that calls back into LLM. The TagData pointer is immutable so we do not have to worry about
		// it disappearing out from under us.
		TagDataMutex.UnlockShared();
		for (const TPair<FTagCreationCallback, UPTRINT>& Callback : GetTagCreationCallbacks())
		{
			Callback.Key(TagData, Callback.Value);
		}
		TagDataMutex.LockShared();
	}
}

TArray<const FTagData*> FLLMGlobals::GetTrackedTagsInner(ELLMTagSet TagSet)
{
	BootstrapInitialise();

	UE::TSharedLock Lock(TagDataMutex);

	TArray<const FTagData*> FoundResults(TagDatas->FilterByPredicate([TagSet](const FTagData* InTagData)
	{
		return InTagData != nullptr && InTagData->GetTagSet() == TagSet;
	}));

	return FoundResults;
}

TArray<const FTagData*> FLLMGlobals::GetTrackedTagsInner(ELLMTracker Tracker, ELLMTagSet TagSet /* = ELLMTagSet::None */)
{
	BootstrapInitialise();

	UE::TUniqueLock UpdateLock(UpdateMutex); // uses of TagSizes are guarded by the UpdateLock
	return GetTracker(Tracker)->GetTagDatas(TagSet);
}

void FLLMGlobals::GetTrackedTagsNamesWithAmountInner(TMap<FName, uint64>& TagsNamesWithAmount, ELLMTracker Tracker, ELLMTagSet TagSet)
{
	BootstrapInitialise();

	UE::TUniqueLock UpdateLock(UpdateMutex);
	GetTracker(Tracker)->GetTagsNamesWithAmount(TagsNamesWithAmount, TagSet);
}

void FLLMGlobals::GetTrackedTagsNamesWithAmountFilteredInner(TMap<FName, uint64>& TagsNamesWithAmount, ELLMTracker Tracker, ELLMTagSet TagSet, TArray<FLLMTagSetAllocationFilter>& Filters)
{
	BootstrapInitialise();

	UE::TUniqueLock UpdateLock(UpdateMutex);
	GetTracker(Tracker)->GetTagsNamesWithAmountFiltered(TagsNamesWithAmount, TagSet, Filters);
}

bool FLLMGlobals::FindTagByNameInner(const TCHAR* Name, uint64& OutTag, ELLMTagSet InTagSet /*= ELLMTagSet::None*/) const
{
	// Cannot call BootstrapInitialise and FinishInitialise without shenanigans because this function is const.
	LLMCheck(bFullyInitialized);

	if (Name != nullptr)
	{
		UE::TSharedLock Lock(TagDataMutex);

		// Search by Name
		FName SearchName(Name);
		FTagData** TagDataPtr = TagDataNameMap->Find(FTagDataNameKey(SearchName, InTagSet));
		if (TagDataPtr)
		{
			const FTagData* TagData = *TagDataPtr;
			OutTag = static_cast<uint64>(TagData->GetContainingEnum());
			return true;
		}

		// Search by ELLMTag's DisplayName
		for (int32 Index = 0; Index < LLM_TAG_COUNT; ++Index)
		{
			const FTagData* TagData = TagDataEnumMap[Index];
			if (!TagData)
			{
				continue;
			}
			if (0 == TCString<TCHAR>::Stricmp(*TagData->GetDisplayName().ToString(), Name))
			{
				OutTag = static_cast<uint64>(TagData->GetContainingEnum());
				return true;
			}
		}
	}

	return false;
}

FName FLLMGlobals::FindTagDisplayNameInner(uint64 Tag) const
{
	// Cannot call BootstrapInitialise without shenanigans because this function is const.
	LLMCheck(bInitializedTracking);

	int32 Index = static_cast<int32>(Tag);
	if (0 <= Index && Index < LLM_TAG_COUNT)
	{
		const FTagData* TagData = TagDataEnumMap[Index];
		if (TagData)
		{
			return TagData->GetDisplayName();
		}
	}
	return NAME_None;
}

FName FLLMGlobals::FindPtrDisplayNameInner(void* Ptr) const
{
	if (!bFullyInitialized)
	{
		return NAME_None;
	}

	const FLLMTracker* TrackerData = GetTracker(ELLMTracker::Default);
	TArray<const FTagData*, TInlineAllocator<static_cast<int32>(ELLMTagSet::Max)>> Tags;
	TrackerData->FindTagsForPtr(Ptr, Tags);

	return Tags.Num() ? Tags[0]->GetDisplayName() : NAME_None;
}

FName FLLMGlobals::GetTagDisplayNameInner(const FTagData* TagData) const
{
	return TagData->GetDisplayName();
}

FString FLLMGlobals::GetTagDisplayPathNameInner(const FTagData * TagData) const
{
	TStringBuilder<FName::StringBufferSize> Buffer;
	GetTagDisplayPathName(TagData, Buffer);
	return FString(Buffer);
}

void FLLMGlobals::GetTagDisplayPathNameInner(const FTagData* TagData,
	FStringBuilderBase& OutPathName, int32 MaxLen) const
{
	TagData->GetDisplayPath(OutPathName, MaxLen);
}

FName FLLMGlobals::GetTagUniqueNameInner(const FTagData* TagData) const
{
	return TagData->GetName();
}

const FTagData* FLLMGlobals::GetTagParentInner(const FTagData* TagData) const
{
	return TagData->GetParent();
}

bool FLLMGlobals::GetTagIsEnumTagInner(const FTagData* TagData) const
{
	return TagData->HasEnumTag();
}

ELLMTag FLLMGlobals::GetTagClosestEnumTagInner(const FTagData* TagData) const
{
	return TagData->GetEnumTag();
}

int64 FLLMGlobals::GetTagAmountForTrackerInner(ELLMTracker Tracker, const FTagData* TagData,
	UE::LLM::ESizeParams SizeParams)
{
	BootstrapInitialise();

	if (TagData == nullptr)
	{
		return 0;
	}

	UE::TUniqueLock UpdateLock(UpdateMutex); // uses of TagSizes are guarded by the UpdateLock
	return GetTracker(Tracker)->GetTagAmount(TagData, SizeParams);
}

int64 FLLMGlobals::GetTagAmountForTrackerInner(ELLMTracker Tracker, FName Tag, ELLMTagSet TagSet,
	UE::LLM::ESizeParams SizeParams)
{
	BootstrapInitialise();

	return GetTagAmountForTracker(Tracker, FindTagData(Tag, TagSet, ETagReferenceSource::FunctionAPI), SizeParams);
}

int64 FLLMGlobals::GetTagAmountForTrackerInner(ELLMTracker Tracker, ELLMTag Tag, UE::LLM::ESizeParams SizeParams)
{
	BootstrapInitialise();

	return GetTagAmountForTracker(Tracker, FindTagData(Tag), SizeParams);
}

void FLLMGlobals::SetTagAmountForTrackerInner(ELLMTracker Tracker, ELLMTag Tag, int64 Amount, bool bAddToTotal)
{
	BootstrapInitialise();
	const FTagData* TagData = FindOrAddTagData(Tag);

	UE::TUniqueLock UpdateLock(UpdateMutex); // uses of TagSizes are guarded by the UpdateLock
	GetTracker(Tracker)->SetTagAmountExternal(TagData, Amount, bAddToTotal);
}

void FLLMGlobals::SetTagAmountForTrackerInner(ELLMTracker Tracker, FName Tag, ELLMTagSet TagSet, int64 Amount, bool bAddToTotal)
{
	BootstrapInitialise();
	const FTagData* TagData = FindOrAddTagData(Tag, TagSet);

	UE::TUniqueLock UpdateLock(UpdateMutex); // uses of TagSizes are guarded by the UpdateLock
	GetTracker(Tracker)->SetTagAmountExternal(TagData, Amount, bAddToTotal);
}

const FTagData* FLLMGlobals::GetActiveTagDataInner(ELLMTracker Tracker, ELLMTagSet TagSet /*= ELLMTagSet::None*/)
{
	BootstrapInitialise();

	return GetTracker(Tracker)->GetActiveTagData(TagSet);
}

uint64 FLLMGlobals::DumpTagInner( ELLMTracker Tracker, const char* FileName, int LineNumber )
{
	BootstrapInitialise();

	const FTagData* TagData = GetActiveTagData(Tracker);
	if (TagData)
	{
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("LLM TAG: %s (%lld) @ %s:%d\n"),
			*TagData->GetDisplayName().ToString(), TagData->GetContainingEnum(),
			FileName ? ANSI_TO_TCHAR(FileName) : TEXT("?"), LineNumber);
		return static_cast<uint64>(TagData->GetContainingEnum());
	}
	else
	{
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("LLM TAG: No Active Tag"));
		return static_cast<uint64>(ELLMTag::Untagged);
	}
}

void FLLMGlobals::PublishDataSingleFrameInner()
{
	if (bAutoPublish)
	{
		UE_LOGF(LogHAL, Error, "Command must be used with the -llmdisableautopublish command line parameter.");
	}
	else
	{
		bPublishSingleFrame = true;
	}
}

const FTagData* FLLMGlobals::FindOrAddTagData(ELLMTag EnumTag, ETagReferenceSource ReferenceSource)
{
	int32 Index = static_cast<int32>(EnumTag);
	LLMCheckf(0 <= Index && Index < LLM_TAG_COUNT, TEXT("Out of range ELLMTag %d"), Index);

	{
		UE::TSharedLock Lock(TagDataMutex);
		FTagData* TagData = TagDataEnumMap[Index];
		if (TagData)
		{
			FinishConstruct(TagData, ReferenceSource);
			return TagData;
		}
	}

	// If we have not initialized tags yet initialize now to potentially create the custom ELLMTag we are reading.
	if (!bFullyInitialized)
	{
		FinishInitialise();
		// Reenter this function so that we retry the find above; we avoid infinite recursion because
		// bFullyInitialized is now true.
		return FindOrAddTagData(EnumTag, ReferenceSource);
	}
	LLMCheckf(!bIsBootstrapping, TEXT("LLM Error: Invalid use of custom ELLMTag when initialising tags."));

	// Add the new Tag
	FName TagName = LLMGetTagUniqueName(EnumTag);
	{
		FTagData* TagData = &RegisterTagData(TagName, NAME_None, NAME_None, NAME_None, NAME_None, true, EnumTag,
			false, ReferenceSource);
		UE::TSharedLock Lock(TagDataMutex);
		FinishConstruct(TagData, ReferenceSource);
		return TagData;
	}
}

const FTagData* FLLMGlobals::FindOrAddTagData(FName TagName, ELLMTagSet TagSet, bool bIsStatTag,
	ETagReferenceSource ReferenceSource)
{
	return FindOrAddTagData(TagName, TagSet, bIsStatTag ? TagName : NAME_None, ReferenceSource);
}

const FTagData* FLLMGlobals::FindOrAddTagData(FName TagName, ELLMTagSet TagSet, FName StatName,
	ETagReferenceSource ReferenceSource)
{
	{
		UE::TSharedLock Lock(TagDataMutex);
		FTagData** TagDataPtr = TagDataNameMap->Find(FTagDataNameKey(TagName, TagSet));
		if (TagDataPtr)
		{
			FTagData* TagData = *TagDataPtr;
			FinishConstruct(TagData, ReferenceSource);
			return TagData;
		}
	}

	// If we have not initialized tags yet initialize now to potentially create the custom ELLMTag we are reading.
	if (!bFullyInitialized)
	{
		FinishInitialise();
		// Reenter this function so that we retry the find above; note we avoid infinite recursion because
		// bFullyInitialized is now true.
		return FindOrAddTagData(TagName, TagSet, StatName, ReferenceSource);
	}
	LLMCheckf(!bIsBootstrapping, TEXT("LLM Error: Invalid use of FName tag when initialising tags."));

	// Add the new Tag
	bool bIsStatTag = !StatName.IsNone() && StatName == TagName;
	FTagData* TagData = &RegisterTagData(TagName, NAME_None, NAME_None, StatName, NAME_None, false,
		ELLMTag::CustomName, bIsStatTag, ReferenceSource, TagSet);
	{
		UE::TSharedLock Lock(TagDataMutex);
		FinishConstruct(TagData, ReferenceSource);
		return TagData;
	}
}

const FTagData* FLLMGlobals::FindTagData(ELLMTag EnumTag, ETagReferenceSource ReferenceSource)
{
	int32 Index = static_cast<int32>(EnumTag);
	LLMCheckf(0 <= Index && Index < LLM_TAG_COUNT, TEXT("Out of range ELLMTag %d"), Index);

	UE::TSharedLock Lock(TagDataMutex);
	FTagData* TagData = TagDataEnumMap[Index];
	if (TagData)
	{
		FinishConstruct(TagData, ReferenceSource);
		return TagData;
	}
	else
	{
		return nullptr;
	}
}

const FTagData* FLLMGlobals::FindTagData(FName TagName, ELLMTagSet TagSet, ETagReferenceSource ReferenceSource)
{
	UE::TSharedLock Lock(TagDataMutex);

	FTagData** TagDataPtr = TagDataNameMap->Find(FTagDataNameKey(TagName, TagSet));
	if (TagDataPtr)
	{
		FTagData* TagData = *TagDataPtr;
		FinishConstruct(TagData, ReferenceSource);
		return TagData;
	}
	else
	{
		return nullptr;
	}
}

const FTagData* FLLMGlobals::FindOrAddDefaultTagData(ELLMTagSet TagSet)
{
	switch (TagSet)
	{
	case ELLMTagSet::None:
		return FindOrAddTagData(ELLMTag::Untagged, ETagReferenceSource::FunctionAPI);
	case ELLMTagSet::Assets:
		return FindOrAddTagData(TagName_UntaggedAsset, TagSet, false /* bIsStatTag */,
			ETagReferenceSource::FunctionAPI);
	case ELLMTagSet::AssetClasses:
		return FindOrAddTagData(TagName_UntaggedAssetClass, TagSet, false /* bIsStatTag */,
			ETagReferenceSource::FunctionAPI);
	case ELLMTagSet::UObjectClasses:
		return FindOrAddTagData(TagName_UntaggedUObjectClasses, TagSet, false /* bIsStatTag */,
			ETagReferenceSource::FunctionAPI);
	case ELLMTagSet::CodeOrContent:
		return FindOrAddCodeOrContentCodeTagData();
	default:
		return nullptr;
	}
}

const FTagData* FLLMGlobals::FindOrAddCodeOrContentCodeTagData()
{
	return FindOrAddTagData(LLMTagDeclaration_CodeOrContent_Code.GetUniqueName(), ELLMTagSet::CodeOrContent,
		false /* bIsStatTag */, ETagReferenceSource::FunctionAPI);
}

const FTagData* FLLMGlobals::FindOrAddCodeOrContentContentTagData()
{
	return FindOrAddTagData(LLMTagDeclaration_CodeOrContent_Content.GetUniqueName(), ELLMTagSet::CodeOrContent,
		false /* bIsStatTag */, ETagReferenceSource::FunctionAPI);
}

FAllocationGroup* FLLMGlobals::FindOrAddAllocationGroup(const FActiveTags& ActiveTags)
{
	// Caller must hold write lock on AllocationGroupsMutex.
	// Caller must NOT be inside AllocationGroupsIndexMutex; this function may need to enter that mutex
	FAllocationGroup*& AllocationGroup = ActiveTagsToAllocationGroup->FindOrAdd(ActiveTags);
	if (!AllocationGroup)
	{
#if LLM_ALLOW_NAMES_TAGS // When !LLM_ALLOW_NAMES_TAGS, allocating AllocationGroups is more tightly constrained.
		if (!AllocationGroupsFreeList->IsEmpty())
		{
			AllocationGroup = AllocationGroupsFreeList->Pop();
		}
		else
		{
			UE::TUniqueLock WriteScopeLockGlobalIndex(AllocationGroupsIndexMutex);
			AllocationGroup = Allocator.New<FAllocationGroup>(AllocationGroups->Num());
			AllocationGroups->Add(AllocationGroup);
		}
#else // LLM_ALLOW_NAMES_TAGS
		// We do not support activating ELLMTagSets other than ELLMTagSet::None when not using FullTags,
		// because we need to assume we can use just the first TagData of the FActiveTags to
		// map from ELLMTag -> FActiveTags.
		LLMCheck(ActiveTags.Num() == 1);

		// We construct the each AllocationGroup to have an index equal to the integer value of its
		// ELLMTagSet::None tag's EnumTag.
		UE::TUniqueLock WriteScopeLockGlobalIndex(AllocationGroupsIndexMutex);
		uint8 Index = static_cast<uint8>(ActiveTags[0]->GetEnumTag());
		while (AllocationGroups->Num() <= Index)
		{
			FAllocationGroup* AddedGroup = Allocator.New<FAllocationGroup>(AllocationGroups->Num());
			AllocationGroups->Add(AddedGroup);
		}
		AllocationGroup = (*AllocationGroups)[Index];
#endif // else !LLM_ALLOW_NAMES_TAGS
		AllocationGroup->Initialize(ActiveTags);
	}
	return AllocationGroup;
}

FAllocationGroup* FLLMGlobals::FindAllocationGroupForCompressedAllocInfo(ELLMTag Tag)
{
#if LLM_ALLOW_NAMES_TAGS
	LLMCheck(false);
	return nullptr;
#else
	UE::TSharedLock ScopeLock(AllocationGroupsIndexMutex);
	uint8 Index = static_cast<uint8>(Tag);
	// The AllocationGroup should have have been allocated by FindOrAddAllocationGroup during creation of the AllocInfo,
	// and inserted at an Index equal to its ELLMTag.
	LLMCheck(AllocationGroups->Num() > Index);
	return (*AllocationGroups)[Index];
#endif
}

} // namespace UE::LLMPrivate

FLLMScope::FLLMScope(ELLMTag TagEnum, bool bInIsStatTag, ELLMTagSet InTagSet, ELLMTracker InTracker, bool bOverride)
{
	using namespace UE::LLMPrivate;

	Init(InTagSet, InTracker, bOverride,
		// ShouldEnable
		[](FLLMGlobals& LLMRef, FLLMTracker* TrackerPtr)
		{
			return true;
		},
		// OnEnable
		[TagEnum, bInIsStatTag, InTagSet, InTracker](FLLMGlobals&, FLLMTracker* TrackerPtr)
		{
			// This scope does not support TagSets (use FLLMScopeDynamic instead), except for the special case of
			// anonymized allocations, which are indicated with a TagSet and ELLMTag::EngineMisc
			LLMCheck((!bInIsStatTag && InTagSet == ELLMTagSet::None) || (TagEnum == ELLMTag::EngineMisc));
			// ELLMTag::FMalloc is a special tag expected to only be used by the Platform Tracker (see header where defined).
			LLMCheck((TagEnum != ELLMTag::FMalloc) || (ELLMTracker::Platform == InTracker));

			TrackerPtr->PushTag(TagEnum, InTagSet);
		});
}

FLLMScope::FLLMScope(FName TagName, bool bInIsStatTag, ELLMTagSet InTagSet, ELLMTracker InTracker, bool bOverride)
{
	using namespace UE::LLMPrivate;

	Init(InTagSet, InTracker, bOverride,
		// ShouldEnable
		[InTagSet](FLLMGlobals& LLMRef, FLLMTracker* TrackerPtr)
		{
			return LLMRef.IsTagSetScopeActiveInner(InTagSet);
		},
		// OnEnable
		[TagName, bInIsStatTag, InTagSet](FLLMGlobals& LLMRef, FLLMTracker* TrackerPtr)
		{
			TrackerPtr->PushTag(TagName, bInIsStatTag, InTagSet);
		});
}

FLLMScope::FLLMScope(const UE::LLMPrivate::FTagData* TagData, bool bInIsStatTag, ELLMTagSet InTagSet,
	ELLMTracker InTracker, bool bOverride)
{
	using namespace UE::LLMPrivate;

	Init(InTagSet, InTracker, bOverride,
		// ShouldEnable
		[InTagSet](FLLMGlobals& LLMRef, FLLMTracker* TrackerPtr)
		{
			return true;
		},
		// OnEnable
		[TagData, bInIsStatTag, InTagSet, this](FLLMGlobals& LLMRef, FLLMTracker* TrackerPtr)
		{
			TrackerPtr->PushTag(TagData, InTagSet);
			if (InTagSet == ELLMTagSet::AssetClasses && LLMRef.IsTagSetRecordingActiveInner(ELLMTagSet::CodeOrContent))
			{
				// AssetClasses scopes also have the responsibility to trigger CodeOrContent scopes, so when
				// pushing onto AssetClasses, also push onto CodeOrContent.
				bPushedCodeOrContent = true;
				if (!TagData)
				{
					TrackerPtr->PushTag(nullptr, ELLMTagSet::CodeOrContent);
				}
				else
				{
					const FTagData* CodeOrContentData = LLMRef.FindOrAddCodeOrContentContentTagData();
					TrackerPtr->PushTag(CodeOrContentData, ELLMTagSet::CodeOrContent);
				}
			}
		});
}

template <typename ShouldEnableType, typename OnEnabledType>
void FLLMScope::Init(ELLMTagSet InTagSet, ELLMTracker InTracker, bool bOverride,
	ShouldEnableType&& ShouldEnable, OnEnabledType&& OnEnable)
{
	using namespace UE::LLMPrivate;

	if (!FLowLevelMemTracker::IsEnabled())
	{
		return;
	}

	// Note that bEnabled=false because the constructor initializes bEnabled=false before calling the Init function.
	// That original initialization of bEnabled=false occurs in the open, but it does not get rolled back because our
	// data is on the stack and the stack is not rolled back. We rely on our data not being rolled back; we have
	// pointers to it from the lambda we pass to PushAbortHandler.
	LLMCheck(!bEnabled);
	FLLMGlobals& LLMRef = FLLMGlobals::GetInner();
	// Scopes can be encountered in PreMain during AutoRTFM startup, before IsOnCurrentTransactionStack is
	// available. Do not call it until after LLM is fully initialized.
	LLMCheck(!LLMRef.bFullyInitialized || !AutoRTFM::IsClosed() || AutoRTFM::IsOnCurrentTransactionStack(this));

	// We run Init in the open, because we want to track LLM even in transactions.
	UE_AUTORTFM_OPEN
	{
		// We have to check IsEnabled() again after calling Get, because the constructor is called
		// from Get, and will set EnabledState=Disabled if the platform doesn't support it.
		if (FLowLevelMemTracker::IsEnabled())
		{
			LLMRef.BootstrapInitialise();

			FLLMTracker* TrackerPtr = LLMRef.GetTracker(InTracker);
			if (bOverride || TrackerPtr->GetActiveTagData(InTagSet) == nullptr)
			{
				if (ShouldEnable(LLMRef, TrackerPtr))
				{
					bEnabled = true;
					Tracker = InTracker;
					TagSet = InTagSet;
					OnEnable(LLMRef, TrackerPtr);
				}
			}
		}
	};

	// But remember that if we abort while we hold the scope, we need to destroy the scope.
	if (bEnabled)
	{
		AutoRTFM::PushOnAbortHandler(this, [this]
			{
				if (bEnabled)
				{
					DestructInTheOpen();
					bEnabled = false;
				}
			});
	}
}

void FLLMScope::Destruct()
{
	check(bEnabled);
	// We run Destruct in the open, because we want to track LLM even in transactions.
	UE_AUTORTFM_OPEN
	{
		DestructInTheOpen();
		bEnabled = false;
	};
	// But remember to pop our on-abort handler (so that we don't try and double Destruct).
	AutoRTFM::PopOnAbortHandler(this);
}

void FLLMScope::DestructInTheOpen()
{
	using namespace UE::LLMPrivate;

	FLLMGlobals& LLMRef = FLLMGlobals::GetInner();
	FLLMTracker* TrackerPtr = LLMRef.GetTracker(Tracker);
	TrackerPtr->PopTag(TagSet);
	if (bPushedCodeOrContent)
	{
		TrackerPtr->PopTag(ELLMTagSet::CodeOrContent);
	}
}

void FLLMScopeDynamic::Init(ELLMTracker InTracker, ELLMTagSet InTagSet)
{
	// Note that bEnabled=false because the constructor initializes bEnabled=false before calling the Init function.
	// That original initialization of bEnabled=false occurs in the open, but it does not get rolled back because our
	// data is on the stack and the stack is not rolled back. We rely on our data not being rolled back; we have
	// pointers to it from the lambda we pass to PushAbortHandler.
	LLMCheck(!bEnabled);
	UE::LLMPrivate::FLLMGlobals& LLMRef = UE::LLMPrivate::FLLMGlobals::GetInner();
	// Scopes can be encountered in PreMain during AutoRTFM startup, before IsOnCurrentTransactionStack is
	// available. Do not call it until after LLM is fully initialized.
	LLMCheck(!LLMRef.bFullyInitialized || !AutoRTFM::IsClosed() || AutoRTFM::IsOnCurrentTransactionStack(this));

	// We run the init in the open, because we want to track LLM even in transactions.
	UE_AUTORTFM_OPEN
	{
		// We have to check IsEnabled() again after calling Get, because the constructor is called
		// from Get, and will set EnabledState=Disabled if the platform doesn't support it.
		if (!FLowLevelMemTracker::IsEnabled())
		{
			return; // Note that depending on config this might exit out of UE_AUTORTFM_OPEN block only
		}

		LLMRef.BootstrapInitialise();
		if (!LLMRef.IsTagSetScopeActiveInner(InTagSet))
		{
			return; // Note that depending on config this might exit out of UE_AUTORTFM_OPEN block only
		}
		bEnabled = true;
		TagData = nullptr;
		CodeOrContentTagData = nullptr;
		Tracker = InTracker;
		TagSet = InTagSet;
	};

	// But remember that if we abort while we hold the scope, we need to destroy the scope.
	if (bEnabled)
	{
		AutoRTFM::PushOnAbortHandler(this, [this]
			{
				if (bEnabled)
				{
					DestructInTheOpen();
					bEnabled = false;
				}
			});
	}
}

bool FLLMScopeDynamic::TryFindTag(FName TagName)
{
	// Like Init, this function can read/write LLM's internal data (and uses non-transactional-safe locks) and
	// therefore we want to run it in the open. There is nothing we need to revert after an abort other than what has
	// already been set up by Init, so no need for any additional AbortHandler.
	bool bNeedAddTag = false;
	UE_AUTORTFM_OPEN
	{
		// AssetClasses scopes also have the responsibility to trigger CodeOrContent scopes, so we might
		// reach this point even if TagSet is not RecordingActive. In the case where the TagSet is not
		// recording, we should not create its tag and should return true from TryFindTag.
		UE::LLMPrivate::FLLMGlobals& LLMRef = UE::LLMPrivate::FLLMGlobals::GetInner();

		// Setting this->TagData and this->CodeOrContentTagData notifies DestructInTheOpen to unpop them,
		// whether called from RTFM AbortHandler or from this class's destructor. We rely on Activate or
		// TryAddTagAndActivate being called to do the corresponding Push after we set these variables.
		if (LLMRef.IsTagSetRecordingActiveInner(TagSet))
		{
			TagData = LLMRef.FindTagData(TagName, TagSet, UE::LLMPrivate::ETagReferenceSource::Scope);
			bNeedAddTag = TagData == nullptr;
		}
		if (TagSet == ELLMTagSet::AssetClasses && LLMRef.IsTagSetRecordingActiveInner(ELLMTagSet::CodeOrContent))
		{
			CodeOrContentTagData = LLMRef.FindOrAddCodeOrContentContentTagData();
		}
		else
		{
			CodeOrContentTagData = nullptr;
		}
	};
	return !bNeedAddTag;
}

bool FLLMScopeDynamic::TryAddTagAndActivate(FName TagName, const ILLMDynamicTagConstructor& Constructor)
{
	// Like Init, this function can read/write LLM's internal data (and uses non-transactional-safe locks) and
	// therefore we want to run it in the open. There is nothing we need to revert after an abort other than what has
	// already been set up by Init, so no need for any additional AbortHandler.
	UE_AUTORTFM_OPEN
	{
		UE::LLMPrivate::FLLMGlobals & LLMRef = UE::LLMPrivate::FLLMGlobals::GetInner();
		FName StatFullName = NAME_None;
#if STATS
		FString StatConstructorName = Constructor.GetStatName();
		if (!StatConstructorName.IsEmpty())
		{
			if (Constructor.NeedsStatConstruction())
			{
				switch (TagSet)
				{
				case ELLMTagSet::None:
					StatFullName = FDynamicStats::CreateMemoryStatId<FStatGroup_STATGROUP_LLMFULL>(
						StatConstructorName).GetName();
					break;
				case ELLMTagSet::Assets:
					StatFullName = FDynamicStats::CreateMemoryStatId<FStatGroup_STATGROUP_LLMAssets>(
						StatConstructorName).GetName();
					break;
				case ELLMTagSet::AssetClasses:
					StatFullName = FDynamicStats::CreateMemoryStatId<FStatGroup_STATGROUP_LLMAssets>(
						StatConstructorName).GetName();
					break;
				case ELLMTagSet::UObjectClasses:
					StatFullName = FDynamicStats::CreateMemoryStatId<FStatGroup_STATGROUP_LLMUObjects>(
						StatConstructorName).GetName();
					break;
				case ELLMTagSet::CodeOrContent:
					StatFullName = FDynamicStats::CreateMemoryStatId<FStatGroup_STATGROUP_LLMCodeOrContent>(
						StatConstructorName).GetName();
					break;
				default:
					checkNoEntry();
					break;
				}
			}
			else
			{
				StatFullName = FName(*StatConstructorName);
			}
		}
#endif
		// Setting this->TagData and this->CodeOrContentTagData notifies DestructInTheOpen to unpop them,
		// whether called from RTFM AbortHandler or from this class's destructor.
		TagData = LLMRef.FindOrAddTagData(TagName, TagSet, StatFullName, UE::LLMPrivate::ETagReferenceSource::Scope);
		LLMRef.GetTracker(Tracker)->PushTag(TagData, TagSet);
		if (CodeOrContentTagData)
		{
			LLMRef.GetTracker(Tracker)->PushTag(CodeOrContentTagData, ELLMTagSet::CodeOrContent);
		}
	};
	return true;
}

void FLLMScopeDynamic::Activate()
{
	// Like Init, this function can read/write LLM's internal data (and uses non-transactional-safe locks) and
	// therefore we want to run it in the open. There is nothing we need to revert after an abort other than what has
	// already been set up by Init, so no need for any additional AbortHandler.
	UE_AUTORTFM_OPEN
	{
		UE::LLMPrivate::FLLMGlobals & LLMRef = UE::LLMPrivate::FLLMGlobals::GetInner();
		if (TagData)
		{
			LLMRef.GetTracker(Tracker)->PushTag(TagData, TagSet);
		}
		if (CodeOrContentTagData)
		{
			LLMRef.GetTracker(Tracker)->PushTag(CodeOrContentTagData, ELLMTagSet::CodeOrContent);
		}
	};
}

void FLLMScopeDynamic::Destruct()
{
	check(bEnabled);
	// We run Destruct in the open, because we want to track LLM even in transactions.
	UE_AUTORTFM_OPEN
	{
		DestructInTheOpen();
		bEnabled = false;
	};
	// But remember to pop our on-abort handler (so that we don't try and double Destruct).
	AutoRTFM::PopOnAbortHandler(this);
}

void FLLMScopeDynamic::DestructInTheOpen()
{
	UE::LLMPrivate::FLLMGlobals& LLMRef = UE::LLMPrivate::FLLMGlobals::GetInner();
	if (TagData)
	{
		LLMRef.GetTracker(Tracker)->PopTag(TagSet);
	}
	if (CodeOrContentTagData)
	{
		LLMRef.GetTracker(Tracker)->PopTag(ELLMTagSet::CodeOrContent);
	}
}

FLLMPauseScope::FLLMPauseScope(FName TagName, bool bIsStatTag, uint64 Amount, ELLMTracker TrackerToPause,
	ELLMAllocType InAllocType)
{
	if (!FLowLevelMemTracker::IsEnabled())
	{
		return;
	}
	Init(TagName, ELLMTag::Untagged, false, bIsStatTag, Amount, TrackerToPause, InAllocType);
}

FLLMPauseScope::FLLMPauseScope(ELLMTag TagEnum, bool bIsStatTag, uint64 Amount, ELLMTracker TrackerToPause,
	ELLMAllocType InAllocType)
{
	if (!FLowLevelMemTracker::IsEnabled())
	{
		return;
	}
	LLMCheck(!bIsStatTag);
	Init(NAME_None, TagEnum, true, false, Amount, TrackerToPause, InAllocType);
}

void FLLMPauseScope::Init(FName TagName, ELLMTag EnumTag, bool bIsEnumTag, bool bIsStatTag, uint64 Amount,
	ELLMTracker TrackerToPause, ELLMAllocType InAllocType)
{
	// Note that bEnabled=false because the constructor initializes bEnabled=false before calling the Init function.
	// That original initialization of bEnabled=false occurs in the open, but it does not get rolled back because our
	// data is on the stack and the stack is not rolled back. We rely on our data not being rolled back; we have
	// pointers to it from the lambda we pass to PushAbortHandler.
	LLMCheck(!bEnabled);
	UE::LLMPrivate::FLLMGlobals& LLMRef = UE::LLMPrivate::FLLMGlobals::GetInner();
	// Scopes can be encountered in PreMain during AutoRTFM startup, before IsOnCurrentTransactionStack is
	// available. Do not call it until after LLM is fully initialized.
	LLMCheck(!LLMRef.bFullyInitialized || !AutoRTFM::IsClosed() || AutoRTFM::IsOnCurrentTransactionStack(this));

	// We run the init in the open, because we want to track LLM even in transactions.
	UE_AUTORTFM_OPEN
	{
		// We have to check IsEnabled() again after calling Get, because the constructor is called
		// from Get, and will set EnabledState=Disabled if the platform doesn't support it.
		if (!FLowLevelMemTracker::IsEnabled())
		{
			return; // Note that depending on config this might exit out of UE_AUTORTFM_OPEN block only
		}

		LLMRef.BootstrapInitialise();
		if (!LLMRef.IsTagSetScopeActiveInner(ELLMTagSet::None))
		{
			return; // Note that depending on config this might exit out of UE_AUTORTFM_OPEN block only
		}

		bEnabled = true;
		PausedTracker = TrackerToPause;
		AllocType = InAllocType;

		for (int32 TrackerIndex = 0; TrackerIndex < static_cast<int32>(ELLMTracker::Max); TrackerIndex++)
		{
			ELLMTracker Tracker = (ELLMTracker)TrackerIndex;

			if (PausedTracker == ELLMTracker::Max || PausedTracker == Tracker)
			{
				if (Amount == 0)
				{
					LLMRef.GetTracker(Tracker)->Pause(InAllocType);
				}
				else
				{
					if (bIsEnumTag)
					{
						LLMRef.GetTracker(Tracker)->PauseAndTrackMemory(EnumTag, static_cast<int64>(Amount), InAllocType);
					}
					else
					{
						LLMRef.GetTracker(Tracker)->PauseAndTrackMemory(TagName, ELLMTagSet::None, bIsStatTag,
							static_cast<int64>(Amount), InAllocType);
					}
				}
			}
		}
	};

	// But remember that if we abort while we hold the scope, we need to destroy the scope.
	if (bEnabled)
	{
		AutoRTFM::PushOnAbortHandler(this, [this]
			{
				if (bEnabled)
				{
					DestructInTheOpen();
					bEnabled = false;
				}
			});
	}
}

void FLLMPauseScope::Destruct()
{
	check(bEnabled);
	// We run Destruct in the open, because we want to track LLM even in transactions.
	UE_AUTORTFM_OPEN
	{
		DestructInTheOpen();
		bEnabled = false;
	};
	// But remember to pop our on-abort handler (so that we don't try and double Destruct).
	AutoRTFM::PopOnAbortHandler(this);
}

void FLLMPauseScope::DestructInTheOpen()
{
	UE::LLMPrivate::FLLMGlobals& LLMRef = UE::LLMPrivate::FLLMGlobals::GetInner();

	for (int32 TrackerIndex = 0; TrackerIndex < static_cast<int32>(ELLMTracker::Max); TrackerIndex++)
	{
		ELLMTracker Tracker = static_cast<ELLMTracker>(TrackerIndex);

		if (PausedTracker == ELLMTracker::Max || Tracker == PausedTracker)
		{
			LLMRef.GetTracker(Tracker)->Unpause(AllocType);
		}
	}
}

FLLMClearScope::FLLMClearScope(ELLMTagSet InTagSet, ELLMTracker InTracker)
	: FLLMScope(nullptr, false, InTagSet, InTracker)
{
}

FLLMScopeFromPtr::FLLMScopeFromPtr(void* Ptr, ELLMTracker InTracker)
{
	using namespace UE::LLMPrivate;

	if (!FLowLevelMemTracker::IsEnabled())
	{
		return;
	}
	if (Ptr == nullptr)
	{
		return;
	}

	// Note that bEnabled=false because the constructor initializes bEnabled=false before calling the Init function.
	// That original initialization of bEnabled=false occurs in the open, but it does not get rolled back because our
	// data is on the stack and the stack is not rolled back. We rely on our data not being rolled back; we have
	// pointers to it from the lambda we pass to PushAbortHandler.
	LLMCheck(!bEnabled);
	FLLMGlobals& LLMRef = FLLMGlobals::GetInner();
	// Scopes can be encountered in PreMain during AutoRTFM startup, before IsOnCurrentTransactionStack is
	// available. Do not  call it until after LLM is fully initialized.
	LLMCheck(!LLMRef.bFullyInitialized || !AutoRTFM::IsClosed() || AutoRTFM::IsOnCurrentTransactionStack(this));

	// We run the constructor in the open, because we want to track LLM even in transactions.
	UE_AUTORTFM_OPEN
	{
		// We have to check IsEnabled() again after calling Get, because the constructor is called
		// from Get, and will set EnabledState=Disabled if the platform doesn't support it.
		if (!FLowLevelMemTracker::IsEnabled())
		{
			return; // Note that depending on config this might exit out of UE_AUTORTFM_OPEN only
		}

		LLMRef.BootstrapInitialise();

		FLLMTracker* TrackerData = LLMRef.GetTracker(InTracker);

		TArray<const FTagData*, TInlineAllocator<static_cast<int32>(ELLMTagSet::Max)>> Tags;
		if (!TrackerData->FindTagsForPtr(Ptr, Tags))
		{
			return; // Note that depending on config this might exit out of UE_AUTORTFM_OPEN only
		}

		bEnabled = true;
		Tracker = InTracker;

		for (int32 TagSetAsInteger = 0; TagSetAsInteger < static_cast<int32>(ELLMTagSet::Max); TagSetAsInteger++)
		{
			if (!LLMRef.IsTagSetScopeActiveInner(static_cast<ELLMTagSet>(TagSetAsInteger)))
			{
				bSetEnabled[TagSetAsInteger] = false;
				continue;
			}

			if (Tags[TagSetAsInteger])
			{
				bSetEnabled[TagSetAsInteger] = true;
				TrackerData->PushTag(Tags[TagSetAsInteger], static_cast<ELLMTagSet>(TagSetAsInteger));
			}
			else
			{
				bSetEnabled[TagSetAsInteger] = false;
			}
		}
	};

	// But remember that if we abort while we hold the scope, we need to destroy the scope.
	if (bEnabled)
	{
		AutoRTFM::PushOnAbortHandler(this, [this]
			{
				if (bEnabled)
				{
					DestructInTheOpen();
					bEnabled = false;
				}
			});
	}
}

void FLLMScopeFromPtr::Destruct()
{
	check(bEnabled);
	// We run Destruct in the open, because we want to track LLM even in transactions.
	UE_AUTORTFM_OPEN
	{
		DestructInTheOpen();
		bEnabled = false;
	};
	// But remember to pop our on-abort handler (so that we don't try and double Destruct).
	AutoRTFM::PopOnAbortHandler(this);
}

void FLLMScopeFromPtr::DestructInTheOpen()
{
	UE::LLMPrivate::FLLMGlobals& LLMRef = UE::LLMPrivate::FLLMGlobals::GetInner();
	for (int32 TagSetAsInteger = 0; TagSetAsInteger < static_cast<int32>(ELLMTagSet::Max); TagSetAsInteger++)
	{
		if (bSetEnabled[TagSetAsInteger])
		{
			LLMRef.GetTracker(Tracker)->PopTag(static_cast<ELLMTagSet>(TagSetAsInteger));
		}
	}
}

FLLMTagDeclaration::FLLMTagDeclaration(const TCHAR* InCPPName, FName InDisplayName, FName InParentTagName, FName InStatName, FName InSummaryStatName, ELLMTagSet InTagSet)
	:CPPName(InCPPName), UniqueName(NAME_None), DisplayName(InDisplayName), ParentTagName(InParentTagName), StatName(InStatName), SummaryStatName(InSummaryStatName)
	, TagSet(InTagSet)
{
	Register();
}

void FLLMTagDeclaration::ConstructUniqueName()
{
	FString NameBuffer(CPPName);
	NameBuffer.ReplaceCharInline(TEXT('_'), TEXT('/'));
	UniqueName = FName(*NameBuffer);
}

namespace UE::LLMPrivate::LLMTagDeclarationInternal
{

TArray<FLLMTagDeclaration::FCreationCallback, TInlineAllocator<1>>& GetCreationCallbacks()
{
	static TArray<FLLMTagDeclaration::FCreationCallback, TInlineAllocator<1>> CreationCallbacks;
	return CreationCallbacks;
}

FLLMTagDeclaration*& GetList()
{
	static FLLMTagDeclaration* List = nullptr;
	return List;
}

} // namespace UE::LLMPrivate::LLMTagDeclarationInternal

void FLLMTagDeclaration::AddCreationCallback(FCreationCallback InCallback)
{
	TArray<FCreationCallback, TInlineAllocator<1>>& CreationCallbacks =
		UE::LLMPrivate::LLMTagDeclarationInternal::GetCreationCallbacks();
	if (CreationCallbacks.Num() >= CreationCallbacks.Max())
	{
		check(false); // We are not allowed to allocate memory here; you need to increase the allocation size.
	}
	else
	{
		CreationCallbacks.Add(InCallback);
	}
}

void FLLMTagDeclaration::ClearCreationCallbacks()
{
	TArray<FCreationCallback, TInlineAllocator<1>>& CreationCallbacks =
		UE::LLMPrivate::LLMTagDeclarationInternal::GetCreationCallbacks();
	CreationCallbacks.Empty();
}

TArrayView<FLLMTagDeclaration::FCreationCallback> FLLMTagDeclaration::GetCreationCallbacks()
{
	return UE::LLMPrivate::LLMTagDeclarationInternal::GetCreationCallbacks();
}

FLLMTagDeclaration* FLLMTagDeclaration::GetList()
{
	return UE::LLMPrivate::LLMTagDeclarationInternal::GetList();
}

void FLLMTagDeclaration::Register()
{
	for (FCreationCallback CreationCallback : GetCreationCallbacks())
	{
		CreationCallback(*this);
	}
	FLLMTagDeclaration*& List = UE::LLMPrivate::LLMTagDeclarationInternal::GetList();
	Next = List;
	List = this;
}

namespace UE::LLMPrivate
{
#if DO_CHECK

bool HandleAssert(bool bLog, const TCHAR* Format, ...)
{
	if (bLog)
	{
		TCHAR DescriptionString[4096];
		GET_TYPED_VARARGS(TCHAR, DescriptionString, UE_ARRAY_COUNT(DescriptionString), UE_ARRAY_COUNT(DescriptionString) - 1,
			Format, Format);

		FPlatformMisc::LowLevelOutputDebugString(DescriptionString);

		if (FPlatformMisc::IsDebuggerPresent())
			FPlatformMisc::PromptForRemoteDebugging(true);

		UE_DEBUG_BREAK();
	}
	return false;
}

#endif

const TCHAR* ToString(UE::LLMPrivate::ETagReferenceSource ReferenceSource)
{
	switch (ReferenceSource)
	{
	case ETagReferenceSource::Scope:
		return TEXT("LLM_SCOPE");
	case ETagReferenceSource::Declare:
		return TEXT("LLM_DEFINE_TAG");
	case ETagReferenceSource::EnumTag:
		return TEXT("LLM_ENUM_GENERIC_TAGS");
	case ETagReferenceSource::CustomEnumTag:
		return TEXT("RegisterPlatformTag/RegisterProjectTag");
	case ETagReferenceSource::FunctionAPI:
		return TEXT("DefaultName/InternalCall");
	case ETagReferenceSource::ImplicitParent:
		return TEXT("ImplicitParent");
	default:
		return TEXT("Invalid");
	}
}

void ValidateUniqueName(FStringView UniqueName)
{
	// Characters that are invalid for c++ names are invalid (other than /), since we use uniquenames
	// (with / replaced by _) as part of the name of the auto-constructed FLLMTagDeclaration variables
	// _ is invalid since we use an _ to indicate a / in FLLMTagDeclaration.
	// So only Alnum characters or / are allowed, and the first character can not be a number.
	if (UniqueName.Len() == 0)
	{
		LLMCheckf(false, TEXT("Invalid length-zero Tag Unique Name"));
	}
	else
	{
		LLMCheckf(!TChar<TCHAR>::IsDigit(UniqueName[0]),
			TEXT("Invalid first character is digit in Tag Unique Name '%.*s'"),
			UniqueName.Len(), UniqueName.GetData());
	}
	for (TCHAR c : UniqueName)
	{
		if (!TChar<TCHAR>::IsAlnum(c) && c != TEXT('/'))
		{
			LLMCheckf(false, TEXT("Invalid character %c in Tag Unique Name '%.*s'"), c,
				UniqueName.Len(), UniqueName.GetData());
		}
	}
}

namespace AllocatorPrivate
{

/**
 * When a Page is allocated, it splits the memory of the page up into blocks, and creates an FAlloc at the start of
 * each block. All the FAllocs are joined together in a FreeList linked list.
 * When a Page allocates memory, it takes an FAlloc from the freelist and gives it to the caller, and forgets about it.
 * When the caller returns a pointer, the Page restores the FAlloc at the beginning of the block and puts it back on
 * the FreeList.
 */
struct FAlloc
{
	FAlloc* Next;
};

/**
 * An FPage holds a single page of memory received from the OS; all pages are of the same size.
 * FPages are owned by FBins, and the FPages for an FBin divide the page up into blocks of the FBin's size.
 * An FPage keeps track of the blocks it has not yet given out so it can allocate, and keeps track of how many blocks
 * it has given out, so that it can be freed when no longer used.
 * Pages that are not free or empty are available for allocating from and are kept in a doubly-linked list on the FBin.
 */
struct FPage
{
	FPage(int32 PageSize, int32 BinSize);
	void* Allocate();
	void Free(void* Ptr);
	bool IsFull() const;
	bool IsEmpty() const;
	void AddToList(FPage*& Head);
	void RemoveFromList(FPage*& Head);

	FAlloc* FreeList;
	FPage* Prev;
	FPage* Next;
	int32 UsedCount;
};

/**
 * An FBin handles all allocations that fit in its size range. Its size is the power of two at the top of that range.
 * The FBin allocates one FPage at a time from the OS; the FPage gets split up into blocks and handles providing a
 * block for callers requesting a pointer.
 * The FBin has a doubly-linked list of pages in use but not yet full. It provides allocations from these pages.
 * When an FPage gets full, the FBin forgets about it, counting on the caller to give the pointer to the page back
 * when it frees the pointer and the page becomes non-full again.
 * When an FBin has no more non-full pages and needs to satisfy an alloc, it allocates a new page.
 * When a page becomes unused due to a free, the FBin frees the page, returning it to the OS.
 */
struct FBin
{
	FBin(int32 InBinSize);
	void* Allocate(FLLMAllocator& Allocator);
	void Free(void* Ptr, FLLMAllocator& Allocator);

	FPage* FreePages;
	int32 UsedCount;
	int32 BinSize;
};

} // namespace AllocatorPrivate

FLLMAllocator*& FLLMAllocator::Get()
{
	static FLLMAllocator* Allocator = nullptr;
	return Allocator;
}

FLLMAllocator::FLLMAllocator()
	: PlatformAlloc(nullptr)
	, PlatformFree(nullptr)
	, Bins(nullptr)
	, Total(0)
	, PageSize(0)
	, NumBins(0)
{
}

FLLMAllocator::~FLLMAllocator()
{
	Clear();
}

void FLLMAllocator::Initialize(LLMAllocFunction InAlloc, LLMFreeFunction InFree, int32 InPageSize)
{
	using namespace UE::LLMPrivate::AllocatorPrivate;

	PlatformAlloc = InAlloc;
	PlatformFree = InFree;
	PageSize = InPageSize;

	if (PlatformAlloc)
	{
		constexpr int32 MinBinSizeForAlignment = 16;
		constexpr int32 MinBinSizeForAllocStorage = static_cast<int32>(sizeof(FAlloc));
		constexpr int32 MultiplierBetweenBins = 2;
		// Setting MultiplierAfterLastBin=2 would be useless because the PageSize/2 bin would only get a single
		// allocation out of each page due to the FPage data taking up the first half
		// TODO: For bins >= 4*FPage size, allocate FPages in a separate list rather than embedding them.
		// This will require allocating extra space in each allocation to store its page pointer.
		constexpr int32 MultiplierAfterLastBin = 4;

		int32 MinBinSize = FMath::Max(MinBinSizeForAllocStorage, MinBinSizeForAlignment);
		int32 MaxBinSize = InPageSize / MultiplierAfterLastBin;
		int32 BinSize = MinBinSize;
		while (BinSize <= MaxBinSize)
		{
			BinSize *= MultiplierBetweenBins;
			++NumBins;
		}
		if (NumBins > 0)
		{
			Bins = reinterpret_cast<FBin*>(AllocPages(NumBins * sizeof(FBin)));
			BinSize = MinBinSize;
			for (int32 BinIndex = 0; BinIndex < NumBins; ++BinIndex)
			{
				new (&Bins[BinIndex]) FBin(BinSize);
				BinSize *= MultiplierBetweenBins;
			}
		}
	}
}

void FLLMAllocator::Clear()
{
	using namespace UE::LLMPrivate::AllocatorPrivate;

	if (NumBins)
	{
		for (int32 BinIndex = 0; BinIndex < NumBins; ++BinIndex)
		{
			LLMCheck(Bins[BinIndex].UsedCount == 0);
			Bins[BinIndex].~FBin();
		}
		FreePages(Bins, NumBins * sizeof(FBin));
		Bins = nullptr;
		NumBins = 0;
	}
}

void* FLLMAllocator::Malloc(size_t Size)
{
	return Alloc(Size);
}

void* FLLMAllocator::Alloc(size_t Size)
{
	using namespace UE::LLMPrivate::AllocatorPrivate;

	if (Size == 0)
	{
		return nullptr;
	}
	int32 BinIndex = GetBinIndex(Size);
	UE::TUniqueLock Lock(Mutex);
	if (BinIndex == NumBins)
	{
		return AllocPages(Size);
	}
	return Bins[BinIndex].Allocate(*this);
}

void FLLMAllocator::Free(void* Ptr, size_t Size)
{
	using namespace UE::LLMPrivate::AllocatorPrivate;

	if (Ptr != nullptr)
	{
		int32 BinIndex = GetBinIndex(Size);
		UE::TUniqueLock Lock(Mutex);
		if (BinIndex == NumBins)
		{
			FreePages(Ptr, Size);
		}
		else
		{
			Bins[BinIndex].Free(Ptr, *this);
		}
	}
}

void* FLLMAllocator::Realloc(void* Ptr, size_t OldSize, size_t NewSize)
{
	void* NewPtr;
	if (NewSize)
	{
		NewPtr = Alloc(NewSize);
		if (OldSize)
		{
			size_t CopySize = FMath::Min(OldSize, NewSize);
			FMemory::Memcpy(NewPtr, Ptr, CopySize);
		}
	}
	else
	{
		NewPtr = nullptr;
	}
	Free(Ptr, OldSize);
	return NewPtr;
}

int64 FLLMAllocator::GetTotal() const
{
	UE::TUniqueLock Lock(const_cast<UE::FPlatformRecursiveMutex&>(Mutex));
	return Total;
}

void* FLLMAllocator::AllocPages(size_t Size)
{
	Size = Align(Size, PageSize);
	void* Ptr = PlatformAlloc(Size);
	LLMCheck(Ptr);
	LLMCheck((reinterpret_cast<intptr_t>(Ptr) & (PageSize - 1)) == 0);
	Total += Size;
	return Ptr;
}

void FLLMAllocator::FreePages(void* Ptr, size_t Size)
{
	Size = Align(Size, PageSize);
	PlatformFree(Ptr, Size);
	Total -= Size;
}

int32 FLLMAllocator::GetBinIndex(size_t Size) const
{
	int BinIndex = 0;
	while (BinIndex < NumBins && static_cast<size_t>(Bins[BinIndex].BinSize) < Size)
	{
		++BinIndex;
	}
	return BinIndex;
}

namespace AllocatorPrivate
{

FPage::FPage(int32 PageSize, int32 BinSize)
{
	Next = Prev = nullptr;
	UsedCount = 0;
	int32 NumHeaderBins = (FMath::Max(static_cast<int32>(sizeof(FPage)), BinSize) + BinSize - 1) / BinSize;
	int32 FreeCount = PageSize / BinSize - NumHeaderBins;

	// Divide the rest of the page after this header into FAllocs, and add all the FAllocs into the free list.
	FreeList = reinterpret_cast<FAlloc*>(reinterpret_cast<intptr_t>(this) + NumHeaderBins*BinSize);
	FAlloc* EndAlloc = reinterpret_cast<FAlloc*>(
	reinterpret_cast<intptr_t>(FreeList) + (FreeCount-1) * BinSize);
	FAlloc* Alloc = FreeList;
	while (Alloc != EndAlloc)
	{
		Alloc->Next = reinterpret_cast<FAlloc*>(reinterpret_cast<intptr_t>(Alloc) + BinSize);
		Alloc = Alloc->Next;
	}
	EndAlloc->Next = nullptr;
}

void* FPage::Allocate()
{
	LLMCheck(FreeList);
	FAlloc* Alloc = FreeList;
	FreeList = Alloc->Next;
	++UsedCount;
	return Alloc;
}

void FPage::Free(void* Ptr)
{
	LLMCheck(UsedCount > 0);
	FAlloc* Alloc = reinterpret_cast<FAlloc*>(Ptr);
	Alloc->Next = FreeList;
	FreeList = Alloc;
	--UsedCount;
}

bool FPage::IsFull() const
{
	return FreeList == nullptr;
}

bool FPage::IsEmpty() const
{
	return UsedCount == 0;
}

void FPage::AddToList(FPage*& Head)
{
	Next = Head;
	Prev = nullptr;
	Head = this;
	if (Next)
	{
		Next->Prev = this;
	}
}

void FPage::RemoveFromList(FPage*& Head)
{
	if (Prev)
	{
		Prev->Next = Next;
		if (Next)
		{
			Next->Prev = Prev;
		}
	}
	else
	{
		Head = Next;
		if (Next)
		{
			Next->Prev = nullptr;
		}
	}
	Next = Prev = nullptr;
}

FBin::FBin(int32 InBinSize)
{
	FreePages = nullptr;
	UsedCount = 0;
	BinSize = InBinSize;
}

void* FBin::Allocate(FLLMAllocator& Allocator)
{
	if (!FreePages)
	{
		FPage* Page = reinterpret_cast<FPage*>(Allocator.AllocPages(Allocator.PageSize));
		++UsedCount;
		LLMCheck(Page);
		// The FPage is at the beginning of the array of PageSize bytes.
		new (Page) FPage(Allocator.PageSize, BinSize);
		Page->AddToList(FreePages);
	}

	void* Result = FreePages->Allocate();
	if (FreePages->IsFull())
	{
		FreePages->RemoveFromList(FreePages); //-V678
	}
	return Result;
}

void FBin::Free(void* Ptr, FLLMAllocator& Allocator)
{
	FPage* Page = reinterpret_cast<FPage*>(
		reinterpret_cast<intptr_t>(Ptr) & ~(static_cast<intptr_t>(Allocator.PageSize) - 1));
	if (Page->IsFull())
	{
		Page->AddToList(FreePages);
	}
	Page->Free(Ptr);
	if (Page->IsEmpty())
	{
		Page->RemoveFromList(FreePages);
		--UsedCount;
		Allocator.FreePages(Page, Allocator.PageSize);
	}
}

} // namespace AllocatorPrivate

FTagData::FTagData(FName InName, ELLMTagSet InTagSet, FName InDisplayName, FName InParentName, FName InStatName,
	FName InSummaryStatName, bool bInHasEnumTag, ELLMTag InEnumTag, ETagReferenceSource InReferenceSource)
	: Name(InName), DisplayName(InDisplayName), ParentName(InParentName), StatName(InStatName)
	, SummaryStatName(InSummaryStatName), EnumTag(InEnumTag), ReferenceSource(InReferenceSource)
	, TagSet(InTagSet), bIsFinishConstructed(false), bParentIsName(true), bHasEnumTag(bInHasEnumTag)
	, bIsReportable(true), bIsTraceReportable(true)
{
}

FTagData::FTagData(FName InName, ELLMTagSet InTagSet, FName InDisplayName, const FTagData* InParent, FName InStatName,
	FName InSummaryStatName, bool bInHasEnumTag, ELLMTag InEnumTag, ETagReferenceSource InReferenceSource)
	: FTagData(InName, InTagSet, InDisplayName, NAME_None, InStatName, InSummaryStatName, bInHasEnumTag, InEnumTag,
		InReferenceSource)
{
	SetParent(InParent);
}

FTagData::FTagData(ELLMTag InEnumTag)
	: FTagData(NAME_None, ELLMTagSet::None, NAME_None, NAME_None, NAME_None, NAME_None, true, InEnumTag,
		ETagReferenceSource::EnumTag)
{
}

FTagData::~FTagData()
{
	if (bParentIsName)
	{
		ParentName.~FName();
		bParentIsName = false;
	}
}

bool FTagData::IsFinishConstructed() const
{
	return bIsFinishConstructed;
}

bool FTagData::IsParentConstructed() const
{
	return !bParentIsName;
}

FName FTagData::GetName() const
{
	return Name;
}

FName FTagData::GetDisplayName() const
{
	return DisplayName;
}

void FTagData::GetDisplayPath(FStringBuilderBase& Result, int32 MaxLen) const
{
	Result.Reset();
	AppendDisplayPath(Result, MaxLen);
}

void FTagData::AppendDisplayPath(FStringBuilderBase& Result, int32 MaxLen) const
{
	if (Parent && Parent->IsUsedAsDisplayParent())
	{
		Parent->AppendDisplayPath(Result, MaxLen);
		if (MaxLen >= 0 && Result.Len() + 1 >= MaxLen)
		{
			return;
		}
		Result << TEXT("/");
	}
	if (MaxLen >= 0)
	{
		int32 MaxRemainingLen = MaxLen - Result.Len();
		if (static_cast<int32>(DisplayName.GetStringLength() + 1) > MaxRemainingLen)
		{
			if (MaxRemainingLen > 1)
			{
				TStringBuilder<FName::StringBufferSize> Buffer;
				DisplayName.AppendString(Buffer);
				Result << Buffer.ToView().Left(MaxRemainingLen - 1);
			}
			return;
		}
	}
	DisplayName.AppendString(Result);
}

const FTagData* FTagData::GetParent() const
{
	LLMCheckf(!bParentIsName, TEXT("GetParent called on TagData %s before SetParent was called"),
		*WriteToString<FName::StringBufferSize>(Name));
	return Parent;
}

FName FTagData::GetParentName() const
{
	LLMCheckf(bParentIsName, TEXT("GetParentName called on TagData %s after SetParent was called"),
		*WriteToString<FName::StringBufferSize>(Name));
	return ParentName;
}

FName FTagData::GetParentNameSafeBeforeFinishConstruct() const
{
	if (bParentIsName)
	{
		return ParentName;
	}
	else
	{
		return Parent ? Parent->GetName() : NAME_None;
	}
}

FName FTagData::GetStatName() const
{
	return StatName;
}

FName FTagData::GetSummaryStatName() const
{
	return SummaryStatName;
}

ELLMTag FTagData::GetEnumTag() const
{
	return EnumTag;
}

ELLMTagSet FTagData::GetTagSet() const
{
	return TagSet;
}

bool FTagData::HasEnumTag() const
{
	return bHasEnumTag;
}

const FTagData* FTagData::GetContainingEnumTagData() const
{
	const FTagData* TagData = this;
	do
	{
		if (TagData->bHasEnumTag)
		{
			return TagData;
		}
		TagData = TagData->GetParent();
	} while (TagData);
	LLMCheckf(false, TEXT("TagData is not a descendant of an ELLMTag TagData. ")
		TEXT("All TagDatas must be descendants of ELLMTag::CustomName if they are not descendants of any other ELLMTag"));
		return this;
}

ELLMTag FTagData::GetContainingEnum() const
{
	const FTagData* TagData = GetContainingEnumTagData();
	return TagData->EnumTag;
}

ETagReferenceSource FTagData::GetReferenceSource() const
{
	return ReferenceSource;
}

int32 FTagData::GetIndex() const
{
	return Index;
}

bool FTagData::IsReportable() const
{
	return bIsReportable && (TagSet == ELLMTagSet::None || TagSet == ELLMTagSet::CodeOrContent);
}

bool FTagData::IsStatsReportable() const
{
	return bIsReportable;
}

bool FTagData::IsTraceReportable() const
{
	return bIsTraceReportable && bIsReportable;
}

void FTagData::SetParent(const FTagData* InParent)
{
	if (bParentIsName)
	{
		ParentName.~FName();
		bParentIsName = false;
	}
	Parent = InParent;
}

void FTagData::SetName(FName InName)
{
	Name = InName;
}

void FTagData::SetDisplayName(FName InDisplayName)
{
	DisplayName = InDisplayName;
}

void FTagData::SetStatName(FName InStatName)
{
	StatName = InStatName;
}

void FTagData::SetSummaryStatName(FName InSummaryStatName)
{
	SummaryStatName = InSummaryStatName;
}

void FTagData::SetParentName(FName InParentName)
{
	LLMCheck(bParentIsName);
	ParentName = InParentName;
}

void FTagData::SetFinishConstructed()
{
	bIsFinishConstructed = true;
}

void FTagData::SetIndex(int32 InIndex)
{
	Index = InIndex;
}

void FTagData::SetIsReportable(bool bInIsReportable)
{
	bIsReportable = bInIsReportable;
}

void FTagData::SetIsTraceReportable(bool bInIsTraceReportable)
{
	bIsTraceReportable = bInIsTraceReportable;
}

bool FTagData::IsUsedAsDisplayParent() const
{
	// All Tags but one are UsedAsDisplayParent - their name is prepended during GetDisplayPath.
	// ELLMTag::CustomName is the exception. It is set for FName tags that do not have a real parent to provide a
	// containing ELLMTag for them to provide to systems that do not support FName tags. When FName tags without a real
	// parent are displayed, their path should display as parentless despite having the CustomName tag as their parent.
	return !(bHasEnumTag && EnumTag == ELLMTag::CustomName);
}

#if LLM_ALLOW_NAMES_TAGS

void FLLMTracker::FLowLevelAllocInfo::SetGroup(FAllocationGroup* AllocationGroup)
{
	Group = AllocationGroup->GetIndex();
}

FAllocationGroup* FLLMTracker::FLowLevelAllocInfo::GetGroup(FLLMGlobals& InLLMRef) const
{
	UE::TSharedLock ScopeLock(InLLMRef.AllocationGroupsIndexMutex);
	return (*InLLMRef.AllocationGroups)[Group];
}

int32 FLLMTracker::FLowLevelAllocInfo::GetCompressedTag() const
{
	return Group;
}

#else // LLM_ALLOW_NAMES_TAGS

void FLLMTracker::FLowLevelAllocInfo::SetGroup(FAllocationGroup* AllocationGroup)
{
	Tag = AllocationGroup->GetActiveTags().GetSystemsTagData()->GetEnumTag();
}

FAllocationGroup* FLLMTracker::FLowLevelAllocInfo::GetGroup(FLLMGlobals& InLLMRef) const
{
	return InLLMRef.FindAllocationGroupForCompressedAllocInfo(Tag);
}

int32 FLLMTracker::FLowLevelAllocInfo::GetCompressedTag() const
{
	return static_cast<int32>(Tag);
}

#endif // else !LLM_ALLOW_NAMES_TAGS

FLLMTracker::FLLMTracker(FLLMGlobals& InLLM)
	: LLMRef(InLLM)
	, Tracker(ELLMTracker::Max)
	, TrackedTotal(0)
	, OverrideUntaggedTagData(nullptr)
	, OverrideTrackedTotalTagData(nullptr)
	, LastTrimTime(0.0)
{
	TlsSlot = FPlatformTLS::AllocTlsSlot();

	for (int32 Index = 0; Index < static_cast<int32>(ELLMAllocType::Count); ++Index)
	{
		AllocTypeAmounts[Index] = 0;
	}
}

FLLMTracker::~FLLMTracker()
{
	Clear();

	FPlatformTLS::FreeTlsSlot(TlsSlot);
}

void FLLMTracker::Initialize(
	ELLMTracker InTracker,
	FLLMAllocator* InAllocator)
{
	check(InTracker != ELLMTracker::Max);
	Tracker = InTracker;
	CsvWriter.SetTracker(InTracker);
	TraceWriter.SetTracker(InTracker);
	CsvProfilerWriter.SetTracker(InTracker);

#if !UE_ONLY_USE_PLATFORM_TRACKER
	AllocationMap.SetAllocator(InAllocator);
#endif
}

FLLMThreadState* FLLMTracker::GetOrCreateState()
{
	// look for already allocated thread state
	FLLMThreadState* State = GetState();
	// Create one if needed
	if (State == nullptr)
	{
		State = LLMRef.Allocator.New<FLLMThreadState>();
		LLMCheckf(State != nullptr, TEXT("LLMRef.Allocator.New returned nullptr."));

		// Add to pending thread states, (these will be consumed on the main thread and transferred to ThreadStates,
		// which is only read/write on main thread). Also add to our backup map from thread id to thread state.
		uint32 ThreadId = FPlatformTLS::GetCurrentThreadId();
		{
			FScopeLock Lock(&PendingThreadStatesGuard);
			PendingThreadStates.Add(State);
			ThreadIdToThreadState.Add(ThreadId, State);
		}

		// push to Tls
		FPlatformTLS::SetTlsValue(TlsSlot, State);
	}
	return State;
}

FLLMThreadState* FLLMTracker::GetState()
{
	FLLMThreadState* State = (FLLMThreadState*)FPlatformTLS::GetTlsValue(TlsSlot);
	if (!State)
	{
		// GetTlsValue might return null even if we previously set it, if called during thread termination
		// Check our backup mapping from thread id to thread state before concluding the state does not exist.
		uint32 ThreadId = FPlatformTLS::GetCurrentThreadId();
		{
			FScopeLock Lock(&PendingThreadStatesGuard);
			FLLMThreadState** ExistingState = ThreadIdToThreadState.Find(ThreadId);
			if (ExistingState)
			{
				State = *ExistingState;
			}
		}
	}
	return State; // Can be nullptr if not yet created
}

void FLLMTracker::PushTag(ELLMTag EnumTag, ELLMTagSet TagSet)
{
	const FTagData* TagData = LLMRef.FindOrAddTagData(EnumTag, ETagReferenceSource::Scope);

	// pass along to the state object
	GetOrCreateState()->PushTag(TagData, TagSet);
}

void FLLMTracker::PushTag(FName Tag, bool bInIsStatData, ELLMTagSet TagSet)
{
	const FTagData* TagData = LLMRef.FindOrAddTagData(Tag, TagSet, bInIsStatData, ETagReferenceSource::Scope);

	// pass along to the state object
	GetOrCreateState()->PushTag(TagData, TagSet);
}

void FLLMTracker::PushTag(const FTagData* TagData, ELLMTagSet TagSet)
{
	// pass along to the state object
	GetOrCreateState()->PushTag(TagData, TagSet);
}

void FLLMTracker::PopTag(ELLMTagSet TagSet)
{
	// look for already allocated thread state
	FLLMThreadState* State = GetState();

	LLMCheckf(State != nullptr, TEXT("Called PopTag but PushTag was never called!"));

	State->PopTag(TagSet);
}

void FLLMTracker::TrackAllocation(FInPtr Ptr, int64 Size, ELLMTag DefaultTag, ELLMAllocType AllocType,
	bool bTrackInMemPro)
{
	FLLMThreadState* State = GetOrCreateState();
	const FTagData* TagData = State->GetTopTag(ELLMTagSet::None);
	bool bPushed = false;
	if (!TagData)
	{
		TagData = LLMRef.FindOrAddTagData(DefaultTag);
		State->PushTag(TagData, ELLMTagSet::None);
		bPushed = true;
	}
	TrackAllocationInActiveTags(Ptr, Size, AllocType, State, bTrackInMemPro);
	if (bPushed)
	{
		State->PopTag(ELLMTagSet::None);
	}
}

void FLLMTracker::TrackAllocation(FInPtr Ptr, int64 Size, FName DefaultTag, ELLMAllocType AllocType,
	bool bTrackInMemPro)
{
	FLLMThreadState* State = GetOrCreateState();
	const FTagData* TagData = State->GetTopTag(ELLMTagSet::None);
	bool bPushed = false;
	if (!TagData)
	{
		TagData = LLMRef.FindOrAddTagData(DefaultTag, ELLMTagSet::None);
		State->PushTag(TagData, ELLMTagSet::None);
		bPushed = true;
	}
	TrackAllocationInActiveTags(Ptr, Size, AllocType, State, bTrackInMemPro);
	if (bPushed)
	{
		State->PopTag(ELLMTagSet::None);
	}
}

void FLLMTracker::TrackAllocationInActiveTags(FInPtr Ptr, int64 Size, ELLMAllocType AllocType,
	FLLMThreadState* State, bool bTrackInMemPro)
{
	if (IsPaused(AllocType))
	{
		// When Paused, we do not track any new allocations and we do not update the counters for the memory
		// they use; the code that triggered the pause is responsible for updating those counters. Since we do not
		// track the allocations, TrackFree will likewise not update the counters when those allocations are freed.
		return;
	}

	// track the total quickly
	FPlatformAtomics::InterlockedAdd(&TrackedTotal, Size);

	// Get the allocationgroup for the current threadstate and increment the threadstate's accumulated size for it.
	FAllocationGroup* AllocationGroup = nullptr;
	State->TrackAllocation(Ptr, Size, LLMRef, Tracker, AllocType, bTrackInMemPro, AllocationGroup);

#if !UE_ONLY_USE_PLATFORM_TRACKER
	// tracking a nullptr with a Size is allowed, but we don't need to remember it, since we can't free it ever.
	if (Ptr != nullptr)
	{
		// remember the size and tag info
		FLLMTracker::FLowLevelAllocInfo AllocInfo;
		AllocInfo.SetGroup(AllocationGroup);
		LLMCheck(Size <= 0x0000'ffff'ffff'ffff);
		uint32 SizeLow = uint32(Size);
		uint16 SizeHigh = uint16(Size >> 32ull);
		PointerKey Key(Ptr, SizeHigh);
		AllocationMap.Add(Key, SizeLow, AllocInfo);
	}
#endif
}

void FLLMTracker::TrackFree(FInPtr Ptr, ELLMAllocType AllocType, bool bTrackInMemPro)
{
#if !UE_ONLY_USE_PLATFORM_TRACKER
	// look up the pointer in the tracking map
	FLLMAllocMap::Values Values;
	{
		if (!AllocationMap.Remove(PointerKey(Ptr), Values))
		{
			return;
		}
	}

	if (IsPaused(AllocType))
	{
		// When Paused, we remove our data for any freed allocations, but we do not update the counters for the
		// memory they used; the code that triggered the pause is responsible for updating those counters.
		return;
	}

	int64 SizeLow = static_cast<int64>(Values.Value1);
	int64 SizeHigh = Values.Key.GetExtraData();
	int64 Size = (SizeHigh << 32ull) | SizeLow;
	FLLMTracker::FLowLevelAllocInfo& AllocInfo = Values.Value2;

	// track the total quickly
	FPlatformAtomics::InterlockedAdd(&TrackedTotal, -Size);

	FLLMThreadState* State = GetOrCreateState();
	FAllocationGroup* AllocationGroup = AllocInfo.GetGroup(LLMRef);
#else
	FLLMThreadState* State = GetOrCreateState();
	int64 Size = 0;
	FAllocationGroup* AllocationGroup = nullptr;
#endif // !UE_ONLY_USE_PLATFORM_TRACKER

	State->TrackFree(Ptr, Size, Tracker, AllocType, AllocationGroup, bTrackInMemPro);
}

void FLLMTracker::OnAllocMoved(FInPtr Dest, FInPtr Source, ELLMAllocType AllocType)
{
#if !UE_ONLY_USE_PLATFORM_TRACKER
	FLLMAllocMap::Values Values;
	{
		if (!AllocationMap.Remove(PointerKey(Source), Values))
		{
			return;
		}

		PointerKey Key(Dest, uint16(Values.Key.GetExtraData()));
		AllocationMap.Add(Key, Values.Value1, Values.Value2);
	}

	if (IsPaused(AllocType))
	{
		// When Paused, don't update counters in case any of the external tracking systems are not available.
		return;
	}

	int64 SizeLow = static_cast<int64>(Values.Value1);
	int64 SizeHigh = Values.Key.GetExtraData();
	int64 Size = (SizeHigh << 32ull) | SizeLow;
	const FLLMTracker::FLowLevelAllocInfo& AllocInfo = Values.Value2;
	FAllocationGroup* AllocationGroup = AllocInfo.GetGroup(LLMRef);
#else
	// TODO: This loses the memory size of an allocation moved within a larger allocation.
	int64 Size = 0;
	const FTagData* TagData = nullptr;
	FAllocationGroup* AllocationGroup = nullptr;
#endif
	FLLMThreadState* State = GetOrCreateState();
	State->TrackMoved(Dest, Source, Size, Tracker, AllocationGroup);
}

void FLLMTracker::TrackMemoryOfActiveTag(int64 Amount, FName DefaultTag, ELLMAllocType AllocType)
{
	FLLMThreadState* State = GetOrCreateState();
	const FTagData* TagData = State->GetTopTag(ELLMTagSet::None);
	if (!TagData)
	{
		TagData = LLMRef.FindOrAddTagData(DefaultTag, ELLMTagSet::None);
	}
	TrackMemoryOfActiveTag(Amount, TagData, AllocType, State);
}

void FLLMTracker::TrackMemoryOfActiveTag(int64 Amount, ELLMTag DefaultTag, ELLMAllocType AllocType)
{
	FLLMThreadState* State = GetOrCreateState();
	const FTagData* TagData = State->GetTopTag(ELLMTagSet::None);
	if (!TagData)
	{
		TagData = LLMRef.FindOrAddTagData(DefaultTag);
	}
	TrackMemoryOfActiveTag(Amount, TagData, AllocType, State);
}

void FLLMTracker::TrackMemoryOfActiveTag(int64 Amount, const FTagData* TagData, ELLMAllocType AllocType, FLLMThreadState* State)
{
	if (IsPaused(AllocType))
	{
		// When Paused, we do not track any delta memory; the code that triggered the pause is responsible for updating the delta memory
		return;
	}

	// track the total quickly
	FPlatformAtomics::InterlockedAdd(&TrackedTotal, Amount);

#if !LLM_ALLOW_NAMES_TAGS
	// When full tags are disabled, we instead store the top-level enumtag parent of the tag used by each allocation
	TagData = TagData->GetContainingEnumTagData();
#endif

	// track on the thread state
	State->TrackMemory(Amount, LLMRef, Tracker, AllocType, TagData);
}

void FLLMTracker::TrackMemory(ELLMTag Tag, int64 Amount, ELLMAllocType AllocType)
{
	TrackMemory(LLMRef.FindOrAddTagData(Tag), Amount, AllocType);
}

void FLLMTracker::TrackMemory(FName Tag, ELLMTagSet TagSet, int64 Amount, ELLMAllocType AllocType)
{
	TrackMemory(LLMRef.FindOrAddTagData(Tag, TagSet, false, ETagReferenceSource::FunctionAPI), Amount, AllocType);
}

void FLLMTracker::TrackMemory(const FTagData* TagData, int64 Amount, ELLMAllocType AllocType)
{
	FLLMThreadState* State = GetOrCreateState();
	FPlatformAtomics::InterlockedAdd(&TrackedTotal, Amount);
	State->TrackMemory(Amount, LLMRef, Tracker, AllocType, TagData);
}

void FLLMTracker::PauseAndTrackMemory(FName TagName, ELLMTagSet TagSet, bool bInIsStatTag, int64 Amount, ELLMAllocType AllocType)
{
	const FTagData* TagData = LLMRef.FindOrAddTagData(TagName, TagSet, bInIsStatTag, ETagReferenceSource::FunctionAPI);
	PauseAndTrackMemory(TagData, Amount, AllocType);
}

void FLLMTracker::PauseAndTrackMemory(ELLMTag EnumTag, int64 Amount, ELLMAllocType AllocType)
{
	const FTagData* TagData = LLMRef.FindOrAddTagData(EnumTag);
	PauseAndTrackMemory(TagData, Amount, AllocType);
}

// This will pause/unpause tracking, and also manually increment a given tag.
void FLLMTracker::PauseAndTrackMemory(const FTagData* TagData, int64 Amount, ELLMAllocType AllocType)
{
	FLLMThreadState* State = GetOrCreateState();
	FPlatformAtomics::InterlockedAdd(&TrackedTotal, Amount);
	State->TrackMemory(Amount, LLMRef, Tracker, AllocType, TagData);
	LLMCheck(State->PausedCounter[static_cast<int32>(AllocType)] < 127);
	State->PausedCounter[static_cast<int32>(AllocType)]++;
}

void FLLMTracker::Pause(ELLMAllocType AllocType)
{
	FLLMThreadState* State = GetOrCreateState();
	LLMCheck(State->PausedCounter[static_cast<int32>(AllocType)] < 127);
	State->PausedCounter[static_cast<int32>(AllocType)]++;
}

void FLLMTracker::Unpause(ELLMAllocType AllocType)
{
	FLLMThreadState* State = GetOrCreateState();
	LLMCheck(State->PausedCounter[static_cast<int32>(AllocType)] > 0);
	State->PausedCounter[static_cast<int32>(AllocType)]--;
}

bool FLLMTracker::IsPaused(ELLMAllocType AllocType)
{
	FLLMThreadState* State = GetState();
	// pause during shutdown, as the external trackers might not be able to robustly handle tracking.
	return IsEngineExitRequested() || (State == nullptr ? false :
		((State->PausedCounter[static_cast<int32>(ELLMAllocType::None)]>0) ||
			(State->PausedCounter[static_cast<int32>(AllocType)])>0));
}

void FLLMTracker::Clear()
{
	{
		FScopeLock Lock(&PendingThreadStatesGuard);
		for (FLLMThreadState* ThreadState : PendingThreadStates)
		{
			LLMRef.Allocator.Delete(ThreadState);
		}
		PendingThreadStates.Empty();
		ThreadIdToThreadState.Empty();
	}

	for (FLLMThreadState* ThreadState : ThreadStates)
	{
		LLMRef.Allocator.Delete(ThreadState);
	}
	ThreadStates.Empty();

#if !UE_ONLY_USE_PLATFORM_TRACKER
	AllocationMap.Clear();
#endif

	CsvWriter.Clear();
	TraceWriter.Clear();
	CsvProfilerWriter.Clear();
}

void FLLMTracker::OnPreFork()
{
	CsvWriter.OnPreFork();
}

void FLLMTracker::SetTotalTags(const FTagData* InOverrideUntaggedTagData,
	const FTagData* InOverrideTrackedTotalTagData)
{
	OverrideUntaggedTagData = InOverrideUntaggedTagData;
	OverrideTrackedTotalTagData = InOverrideTrackedTotalTagData;
}

void FLLMTracker::CaptureTagSnapshot()
{
	UpdateThreads();
	FetchAndClearTagSizes(false /* bUpdatePruning */);

	for (TPair<const FTagData*, FTrackerTagSizeData>& It : TagSizes)
	{
		FTrackerTagSizeData& TrackerInfo = It.Value;
		TrackerInfo.CaptureSnapshot();
	}

	TrackedTotalInSnapshot = TrackedTotal;
}

void FLLMTracker::ClearTagSnapshot()
{
	for (TPair<const FTagData*, FTrackerTagSizeData>& It : TagSizes)
	{
		FTrackerTagSizeData& TrackerInfo = It.Value;
		TrackerInfo.ClearSnapshot();
	}

	TrackedTotalInSnapshot = 0;
}

void FLLMTracker::Update()
{
	UpdateThreads();

	// Add the values from each thread to the central repository.
	FetchAndClearTagSizes(true /* bUpdatePruning */);

	// Update peak sizes and external sizes in the central repository.
	for (TPair<const FTagData*, FTrackerTagSizeData>& It : TagSizes)
	{
		FTrackerTagSizeData& AllocationData = It.Value;

		// Update external amount
		if (AllocationData.bExternalValid)
		{
			if (AllocationData.bExternalAddToTotal)
			{
				FPlatformAtomics::InterlockedAdd(&TrackedTotal, AllocationData.ExternalAmount - AllocationData.Size);
			}
			AllocationData.Size = AllocationData.ExternalAmount;
			AllocationData.bExternalValid = false;
		}

		// Calculate peaks.
#if LLM_ENABLED_TRACK_PEAK_MEMORY
		// TODO: we should keep track of the intra-frame memory peak for the total tracked memory. For now we will
		// use the memory at the time the update happens since there are threading implications to being accurate.
		AllocationData.PeakSize = FMath::Max(AllocationData.PeakSize, AllocationData.Size);
#endif
	}
}

void FLLMTracker::FetchAndClearTagSizes(bool bUpdatePruning)
{
	EPruningLevel PruningLevel = EPruningLevel::None;
	if (bUpdatePruning)
	{
		PruningLevel = EPruningLevel::Default;
		double CurrentTime = FPlatformTime::Seconds();
		constexpr double UpdateTrimPeriod = 10.0;
		bool bTrimAllocations = CurrentTime - LastTrimTime > UpdateTrimPeriod;
		if (bTrimAllocations)
		{ 
			PruningLevel = EPruningLevel::Max;
			LastTrimTime = CurrentTime;
		}
	}

	// Note that we are NOT halting all threads to take a snapshot; we want to avoid doing that because it is bad for
	// performance to make all threads wait while we traverse possibly-large containers on all threads. But this means
	// we can not measure an accurate aggregation of the ActiveAllocs, because
	// 
	// UpdateThread: ReadThread 1: 0 Allocs
	// UpdateThread: ReadThreads [2, K-1]: 0 Allocs
	// Thread1: Make Allocation -> ActiveAllocs++
	// ThreadK: Delete pointer that was allocated from Thread1 -> ActiveAllocs--
	// UpdateThread: ReadThread K, -1 Allocs
	// Our counter is now -1 allocations, when it should be 0.
	// 
	// We therefore need to tolerate ActiveAllocs that can go negative.
	// We can afford to tolerate that, however, because we also have each thread recording its own value of
	// GetUpdatesSinceLastAllocationChange, and clearing that to 0 everytime it sees an allocation or free. If we see
	// two updates in a row with no AllocationChange on all threads, then we know that we have an accurate measure of
	// the ActiveAllocs. And we do not take any control flow action (most notably, we do not prune an AllocationGroup)
	// until both its ThreadRefCount and its ActiveAllocsCount reach 0. And ThreadRefCount can only reach zero when all
	// threads Release, which they only do after PruneUpdatesForTrackingData occurrences of the Update
	// function with no AllocationChange, and we require PruneUpdatesForTrackingData >= 2.
	FMapActiveTagsToAllocationGroupTrackingData LocalTrackings;
	LocalTrackings.Reserve(1000);
	for (FLLMThreadState* ThreadState : ThreadStates)
	{
		ThreadState->FetchAndClearTagSizes(LocalTrackings, AllocTypeAmounts, PruningLevel);
	}

	// System tags (ELLMTagSet::None) are hierarchical, with the size of parent tags being increased by the sizes of any child
	// tags under them. Copy sizes for AllocationGroups with child ELLMTagSet::None tags into the AllocationGroup for the
	// parent ELLMTagSet::None tags and the unchanged tags for other TagSets. We do not support a hierarchy for other
	// TagSets because we haven't needed it so far, and solving the propagation problem for multiple parent hierarchies
	// would be difficult due to the need to handle diamond-shaped inheritance.

	// Add a tracking data for any parent not already in the list. First identify the starting set to add - the parents
	// of tags in the list that are not already in the list.
	TLLMSet<FActiveTags> ParentsToAdd;
	for (TPair<FActiveTags, FAllocationGroupTrackingData>& LocalTrackingPair : LocalTrackings)
	{
		const FActiveTags& ActiveTags = LocalTrackingPair.Key;
		FAllocationGroupTrackingData& LocalTracking = LocalTrackingPair.Value;
		// We rely on all of the elements in LocalTracking having a group when adding Sizes below.
		LLMCheck(LocalTracking.IsInitialized() && LocalTracking.GetGroup() != nullptr);

		const FTagData* ParentSystemsTag = ActiveTags.GetSystemsTagData()->GetParent();
		if (!ParentSystemsTag)
		{
			continue;
		}
		FActiveTags ParentTags = ActiveTags;
		ParentTags.SetSystemsTagData(ParentSystemsTag);
		if (LocalTrackings.Contains(ParentTags))
		{
			continue;
		}

		if (ParentsToAdd.IsEmpty())
		{
			ParentsToAdd.Reserve(100);
		}
		ParentsToAdd.Add(ParentTags);
	}

	// Add a trackingdata for all parents and ancestors in our ParentsToAddList that are not already added.
	// First handle adding parents that already exist in LLMRef.ActiveTagsToAllocationGroup, which we can do under a read lock.
	TLLMSet<FActiveTags> ParentsToAddUnderWriteLock;
	{
		UE::TSharedLock ReadScopeLock(LLMRef.AllocationGroupsMutex);
		for (const FActiveTags& DirectParentTags : ParentsToAdd)
		{
			FAllocationGroupTrackingData* AncestorTracking = &LocalTrackings.FindOrAdd(DirectParentTags);
			if (AncestorTracking->IsInitialized())
			{
				continue;
			}

			FActiveTags AncestorTags = DirectParentTags;
			for (;;)
			{
				FAllocationGroup** AncestorGroupPtr = LLMRef.ActiveTagsToAllocationGroup->Find(AncestorTags);
				if (!AncestorGroupPtr)
				{
					if (ParentsToAddUnderWriteLock.IsEmpty())
					{
						ParentsToAddUnderWriteLock.Reserve(100);
					}
					ParentsToAddUnderWriteLock.Add(AncestorTags);
					break;
				}
				FAllocationGroup* AncestorGroup = *AncestorGroupPtr;

				AncestorTracking->ConditionalInitialize(AncestorGroup);
				const FTagData* AncestorSystemsTag = AncestorTags.GetSystemsTagData()->GetParent();
				if (!AncestorSystemsTag)
				{
					break;
				}

				AncestorTags.SetSystemsTagData(AncestorSystemsTag);
				AncestorTracking = &LocalTrackings.FindOrAdd(AncestorTags);
				if (AncestorTracking->IsInitialized())
				{
					break;
				}
			}
		}
	}

	// Add a trackingdata for all parents and ancestors in our ParentsToAddList that do not already exist in
	// LLMRef.ActiveTagsToAllocationGroup; this requires a WriteLock to create the AllocationGroup.
	{
		UE::TUniqueLock WriteScopeLock(LLMRef.AllocationGroupsMutex);
		for (const FActiveTags& DirectParentTags : ParentsToAddUnderWriteLock)
		{
			FAllocationGroupTrackingData* AncestorTracking = &LocalTrackings.FindOrAdd(DirectParentTags);
			if (AncestorTracking->IsInitialized())
			{
				continue;
			}

			FActiveTags AncestorTags = DirectParentTags;
			for (;;)
			{
				FAllocationGroup* AncestorGroup = LLMRef.FindOrAddAllocationGroup(AncestorTags);

				AncestorTracking->ConditionalInitialize(AncestorGroup);
				const FTagData* AncestorSystemsTag = AncestorTags.GetSystemsTagData()->GetParent();
				if (!AncestorSystemsTag)
				{
					break;
				}

				AncestorTags.SetSystemsTagData(AncestorSystemsTag);
				AncestorTracking = &LocalTrackings.FindOrAdd(AncestorTags);
				if (AncestorTracking->IsInitialized())
				{
					break;
				}
			}
		}
	}

	// Sort the TrackingDatas from leaf to root index of their ELLMTag::None tags, so that we can accumulate
	// Allocation deltas by adding it to one parent at a time. FTagData->GetIndex() is sorted from root to
	// leaf index, so we can use that for the leaf to root index.
	// Note that when in this function we are outside of all LLM mutexes except for the UpdateLock, so it is
	// okay if the sort function allocates memory.
	LocalTrackings.KeySort([](const FActiveTags& A, const FActiveTags& B)
		{
			return A.GetSystemsTagData()->GetIndex() > B.GetSystemsTagData()->GetIndex();
		});

	// Iterate over every TrackingData we have in LocalTrackings, and add its Size and Allocs to its own group and to
	// the TrackingData for its parent group.
	for (TPair<FActiveTags, FAllocationGroupTrackingData>& LocalTrackingPair : LocalTrackings)
	{
		const FActiveTags& ActiveTags = LocalTrackingPair.Key;
		FAllocationGroupTrackingData& LocalTracking = LocalTrackingPair.Value;
		// All TrackingDatas we received from ThreadState->FetchAndClearTagSizes were guaranteed initialized, and we
		// initialized each of the parents that we added in one of the two loops above, so we can assert all elements
		// in LocalTracking are intialized by now.
		LLMCheck(LocalTracking.IsInitialized());
		LLMCheck(LocalTracking.GetGroup() != nullptr);
		FetchAndClearTagSizes_UpdateGroupAllocs(*LocalTracking.GetGroup(), LocalTracking, PruningLevel);

		if (LocalTracking.GetSize() == 0 && LocalTracking.GetActiveAllocs() == 0)
		{
			continue;
		}

		const FTagData* ParentSystemsTag = ActiveTags.GetSystemsTagData()->GetParent();
		if (!ParentSystemsTag)
		{
			continue;
		}

		FActiveTags ParentTags = ActiveTags;
		ParentTags.SetSystemsTagData(ParentSystemsTag);
		FAllocationGroupTrackingData* ParentTracking = LocalTrackings.Find(ParentTags);
		// We added the Tracking data for ParentTags in one of the two loops above, so assert that we have it.
		LLMCheck(ParentTracking && ParentTracking->IsInitialized());

		// Accumulate the Size and ActiveAllocs up to the parent
		// Note: the ActiveAllocs count for a parent tag might incorrectly be zero, without the parent
		// tag being referenced from any thread. So without intervention here we might decide to prune it because
		// it is not referenced. This will not be a behavior problem, because we will just recreate it (with
		// the correct aggregate value of 0 ActiveAllocs and 0 ThreadRefCount) when it is next referenced,
		// but that will be a performance problem. To prevent that problem, ResetUpdatesSinceLastReference whenever
		// a tag is referenced as a parent during in update.
		ParentTracking->GetGroup()->ResetUpdatesSinceLastReference();
		ParentTracking->TrackAllocOrFree(LocalTracking.GetSize(), LocalTracking.GetActiveAllocs());
	}

	if (PruningLevel > EPruningLevel::None)
	{
#if !UE_ONLY_USE_PLATFORM_TRACKER
		if (PruningLevel >= EPruningLevel::Max)
		{
			AllocationMap.Trim();
		}
#endif

		if (this->Tracker == ELLMTracker::Default)
		{
			// Prune the AllocationGroups in the unreferencedlist
			TArray<FAllocationGroup*, FDefaultLLMAllocator> GroupsToFree;
			for (auto GroupIter = LLMRef.AllocationGroupsUnreferencedSet->CreateIterator(); GroupIter; ++GroupIter)
			{
				FAllocationGroup* Group = *GroupIter;
				if (Group->IsReferenced())
				{
					GroupIter.RemoveCurrent();
					continue;
				}
				Group->IncrementUpdatesSinceLastReference();
				if (Group->GetUpdatesSinceLastReference() >= PruneUpdatesForGroup)
				{
					GroupsToFree.Add(Group);
					GroupIter.RemoveCurrent();
					continue;
				}
			}
			if (!GroupsToFree.IsEmpty())
			{
				UE::TUniqueLock WriteScopeLock(LLMRef.AllocationGroupsMutex);
				for (FAllocationGroup* Group : GroupsToFree)
				{
					// Check RefCount again while under the WriteScopeLock. It might have changed in between our check
					// of it outside the lock and now, because a thread can lookup the group from
					// ActiveTagsToAllocationGroup and increment the refcount. We do not have to handle it changing
					// after this point, because it only increments RefCount when inside the AllocationGroupsMutex.
					if (Group->IsReferenced())
					{
						// The group no longer needs to be freed and also no longer needs to be in the
						// AllocationGroupsUnreferencedSet.
						// Drop it from our consideration and keep it in ActiveTagsToAllocationGroup.
						continue;
					}

					// We do not support Resetting AllocationGroups in the !LLM_ALLOW_NAMES_TAGS case, because it's unnecessary since there
					// are so few of them and because FindOrAddAllocationGroup does not yet implement allocating them from the freelist
					// while preserving their Index correctly.
#if LLM_ALLOW_NAMES_TAGS
					LLMRef.ActiveTagsToAllocationGroup->Remove(Group->GetActiveTags());
					Group->Reset();
					LLMRef.AllocationGroupsFreeList->Add(Group);
#endif // LLM_ALLOW_NAMES_TAGS
				}
			}

			if (PruningLevel >= EPruningLevel::Max)
			{
				UE::TUniqueLock WriteScopeLock(LLMRef.AllocationGroupsMutex);
				LLMRef.ActiveTagsToAllocationGroup->Shrink();
				LLMRef.AllocationGroupsUnreferencedSet->Shrink();
				LLMRef.AllocationGroupsFreeList->Shrink();
				{
					UE::TUniqueLock WriteScopeLockGlobalIndex(LLMRef.AllocationGroupsIndexMutex);
					LLMRef.AllocationGroups->Shrink();
				}
			}
		}
	}
}

void FLLMTracker::FetchAndClearTagSizes_UpdateGroupAllocs(FAllocationGroup& AllocationGroup,
	const FAllocationGroupTrackingData& TrackingData, EPruningLevel PruningLevel)
{
	int64 Size = TrackingData.GetSize();
	AllocationGroup.AddSizeAndAllocReferences(this->Tracker, Size, TrackingData.GetActiveAllocs(),
		PruningLevel);
	if (Size != 0)
	{
		// AllocationGroup sizes are accumulated upwards into parent AllocationGroups, where a parent
		// AllocationGroup is the AllocationGroup with all tags the same except for Systems tag which has
		// been replaced with its parent. When copying sizes from AllocationGroup to Tag, for all the
		// other non-hierarchical tags, we need to only copy the sizes from the topmost parent, to avoid
		// counting multiple times the size from a child AllocationGroup that has been added to all of
		// its ancestor group's sizes.
		const FActiveTags& ActiveTags = AllocationGroup.GetActiveTags();
		int32 NumActiveTags = ActiveTags.Num();
		const FTagData* SystemsTag = ActiveTags.GetSystemsTagData();
		LLMCheck(SystemsTag == ActiveTags[0]);
		TagSizes.FindOrAdd(SystemsTag).Size += Size;
		if (SystemsTag->GetParent() == nullptr)
		{
			for (int32 TagIndex = 1; TagIndex < NumActiveTags; ++TagIndex)
			{
				const FTagData* NonHierarchicalTag = ActiveTags[TagIndex];
				TagSizes.FindOrAdd(NonHierarchicalTag).Size += Size;
			}
		}
	}

	if (PruningLevel > EPruningLevel::None)
	{
		if (!AllocationGroup.IsReferenced())
		{
			LLMRef.AllocationGroupsUnreferencedSet->Add(&AllocationGroup);
		}
	}
}

void FLLMTracker::UpdateThreads()
{
	// Consume pending thread states. We must be careful to do all allocations outside of the
	// PendingThreadStatesGuard guard as that can lead to a deadlock due to contention with
	// PendingThreadStatesGuard & Locks inside the underlying allocator (i.e. MallocBinned2 -> Mutex).
	{
		PendingThreadStatesGuard.Lock();
		int32 ReservationSize = PendingThreadStates.Num();
		if (ReservationSize > 0)
		{
			// Exit the lock so we can do the allocation, and then reenter the lock. Keep looping until the
			// Reservation we allocate outside the lock is equal to the reservation we see we need while when we
			// reenter the lock.
			for (;;)
			{
				PendingThreadStatesGuard.Unlock();
				ThreadStates.Reserve(ThreadStates.Num() + ReservationSize);
				PendingThreadStatesGuard.Lock();

				if (PendingThreadStates.Num() != ReservationSize)
				{
					ReservationSize = PendingThreadStates.Num();
					continue;
				}

				ThreadStates.Append(PendingThreadStates);
				PendingThreadStates.Reset();
				break;
			}
		}
		PendingThreadStatesGuard.Unlock();
	}
}

void FLLMTracker::PublishStats(UE::LLM::ESizeParams SizeParams)
{
	if (OverrideTrackedTotalTagData)
	{
		SetMemoryStatByFName(OverrideTrackedTotalTagData->GetStatName(), TrackedTotal);
		SetMemoryStatByFName(OverrideTrackedTotalTagData->GetSummaryStatName(), TrackedTotal);
	}

	if (OverrideUntaggedTagData)
	{
		const FTagData* TagData = LLMRef.FindTagData(TagName_Untagged, ELLMTagSet::None);
		const FTrackerTagSizeData* AllocationData = TagData ? TagSizes.Find(TagData) : nullptr;
		SetMemoryStatByFName(OverrideUntaggedTagData->GetStatName(),
			AllocationData ? AllocationData->GetSize(SizeParams) : 0);
		SetMemoryStatByFName(OverrideUntaggedTagData->GetSummaryStatName(),
			AllocationData ? AllocationData->GetSize(SizeParams) : 0);
	}

	for (const TPair<const FTagData*, FTrackerTagSizeData>& It : TagSizes)
	{
		const FTagData* TagData = It.Key;
		if (!TagData->IsStatsReportable())
		{
			continue;
		}
		if (OverrideUntaggedTagData && TagData->GetName() == TagName_Untagged)
		{
			// Handled separately by OverrideUntaggedTagData.
			continue;
		}
		const FTrackerTagSizeData& AllocationData = It.Value;
		int64 Amount = AllocationData.GetSize(SizeParams);

		SetMemoryStatByFName(TagData->GetStatName(), Amount);
		SetMemoryStatByFName(TagData->GetSummaryStatName(), Amount);
	}
}

static bool PageLessThan(const FForkedPageAllocation& LHS, const FForkedPageAllocation& RHS)
{
	return LHS.PageStart < RHS.PageStart;
}
static bool PageFinder(const FForkedPageAllocation& LHS, const uint64 AddressRHS)
{
	return LHS.PageStart < AddressRHS;
}

bool FLLMTracker::DumpForkedAllocationInfo()
{
#if UE_ONLY_USE_PLATFORM_TRACKER
	UE_LOGF(LogHAL, Error, "UE_ONLY_USE_PLATFORM_TRACKER=1 is defined so reading Allocations from LLM is not available.");
	return false;
#else
	// Try to associate the allocations with a page range so we can determine whether it's
	// in unique or shared memory. Then print out the set for each tag.
	TArray<FForkedPageAllocation> Pages;
	if (FGenericPlatformMemory::GetForkedPageAllocationInfo(Pages) == false)
	{
		// Create a Placeholder page so we can test on platforms that don't support forkedpages
		FForkedPageAllocation& Page = Pages.Emplace_GetRef();
		Page.PageStart = 0;
		Page.PageEnd = MAX_uint64;
		Page.SharedCleanKiB = 1024 * 100;
		Page.SharedDirtyKiB = 1024 * 110;
		Page.PrivateCleanKiB = 1024 * 120;
		Page.PrivateDirtyKiB = 1024 * 130;
	}

	Algo::Sort(Pages, PageLessThan);

	enum EClassification
	{
		Class_Split,
		Class_Unreferenced,
		Class_Private,
		Class_Shared,
		Class_COUNT
	};

	struct FCounts
	{
		uint64 TotalAllocations = 0;
		uint64 CrossPageAllocationCount = 0;

		uint64 AllocCount[Class_COUNT] = {};

		uint64 TotalBytes = 0;
		uint64 ByteCount[Class_COUNT] = {};
	};


	TMap<const FTagData*, FCounts, FDefaultSetLLMAllocator> CountsPerTag;
	{
		UE::TSharedLock Lock(LLMRef.TagDataMutex);
		CountsPerTag.Reserve(LLMRef.TagDatas->Num());
	}

	enum class EErrorType
	{
		NotInPagesList,
		InvalidAllocationGroup,
		InvalidSystemsTag,
		InvalidSize,
		ExtendedBeyondPages,
	};
	struct FErrorData
	{
		uint64 Ptr;
		EErrorType ErrorType;
	};
	TArray<FErrorData, FDefaultLLMAllocator> ErrorPtrs;
	EnumerateAllocations([&Pages, &ErrorPtrs, &CountsPerTag](void* Ptr, int64 InSize, FAllocationGroup* AllocationGroup)
		{
			int FirstPageEqualOrAfterPtr = Algo::LowerBound(Pages, (uint64)Ptr, PageFinder);
			int ContainingAllocationIndex;
			if (FirstPageEqualOrAfterPtr < Pages.Num() && Pages[FirstPageEqualOrAfterPtr].PageStart == (uint64)Ptr)
			{
				ContainingAllocationIndex = FirstPageEqualOrAfterPtr;
			}
			else
			{
				ContainingAllocationIndex = FirstPageEqualOrAfterPtr - 1;
			}
			if (ContainingAllocationIndex < 0 || Pages[ContainingAllocationIndex].PageEnd <= (uint64)Ptr)
			{
				ErrorPtrs.Add({ (uint64)Ptr, EErrorType::NotInPagesList });
				return;
			}
			if (AllocationGroup == nullptr)
			{
				ErrorPtrs.Add({ (uint64)Ptr, EErrorType::InvalidAllocationGroup });
				return;
			}
			const FTagData* SystemsTag = AllocationGroup->GetActiveTags().GetSystemsTagData();
			if (SystemsTag == nullptr)
			{
				ErrorPtrs.Add({ (uint64)Ptr, EErrorType::InvalidSystemsTag });
				return;
			}
			if (InSize < 0)
			{
				ErrorPtrs.Add({ (uint64)Ptr, EErrorType::InvalidSize });
				return;
			}
			// We unfortunately don't know exactly which of the details apply because we don't know which pages
			// in the allocation section are shared/unique etc, so we just keep some generalities.
			uint64 Size = static_cast<uint64>(InSize);

			// Number of page allocations for the ue allocation
			uint32 PageAllocationCount = 0;

			// Track the classification for each page allocation so we can try to classify the ue allocation
			uint32 SplitCount = 0;
			uint32 SharedCount = 0;
			uint32 PrivateCount = 0;
			uint32 UnreferencedCount = 0;

			// Walk the page allocations until we get them all if we aren't all in the same one.
			uint64 Remaining = Size;
			while (Remaining)
			{
				if (ContainingAllocationIndex >= Pages.Num())
				{
					ErrorPtrs.Add({ (uint64)Ptr, EErrorType::ExtendedBeyondPages });
					break;
				}

				PageAllocationCount++;
				FForkedPageAllocation& Page = Pages[ContainingAllocationIndex];

				uint64 PageKiB = (Page.PageEnd - Page.PageStart) / 1024;

				uint64 OffsetInPage = (uint64)Ptr - Page.PageStart;
				uint64 RemainingInPage = Page.PageEnd - (uint64)Ptr;

				uint64 AmountInThisPage = Remaining;
				if (AmountInThisPage > RemainingInPage)
				{
					AmountInThisPage = RemainingInPage;
				}

				uint64 SharedKiB = Page.SharedCleanKiB + Page.SharedDirtyKiB;
				uint64 PrivateKiB = Page.PrivateCleanKiB + Page.PrivateDirtyKiB;
				uint64 UnreferencedKiB = PageKiB - SharedKiB - PrivateKiB;

				// We can't classify the page if we have more than one of any.
				uint64 TypeCount = !!SharedKiB + !!PrivateKiB + !!UnreferencedKiB;
				bool bIsSplit = TypeCount > 1;

				if (bIsSplit)
				{
					// The page is split and we have no way to know which is ours.
					SplitCount++;
				}
				else if (SharedKiB)
				{
					SharedCount++;
				}
				else if (PrivateKiB)
				{
					PrivateCount++;
				}
				else
				{
					UnreferencedCount++;
				}

				Remaining -= AmountInThisPage;
				ContainingAllocationIndex++;
			}

			if (Remaining)
			{
				// We error'd out, ignore this allocation.
				return;
			}

			// Classify the allocation - is it entirely private, entirely shared, or what.
			uint64 ClassCount = !!SharedCount + !!PrivateCount + !!UnreferencedCount;
			EClassification AllocClass = Class_Split;
			if (ClassCount > 1)
			{
				AllocClass = Class_Split;
			}
			else if (SharedCount)
			{
				AllocClass = Class_Shared;
			}
			else if (PrivateCount)
			{
				AllocClass = Class_Private;
			}
			else
			{
				AllocClass = Class_Unreferenced;
			}

			// Associate the tag and the page stats.
			FCounts& Counts = CountsPerTag.FindOrAdd(SystemsTag);
			Counts.AllocCount[AllocClass]++;
			Counts.ByteCount[AllocClass] += Size;
			Counts.TotalAllocations++;
			Counts.TotalBytes += Size;
			if (PageAllocationCount > 1)
			{
				Counts.CrossPageAllocationCount++;
			}
		});
	for (FErrorData& ErrorData : ErrorPtrs)
	{
		switch (ErrorData.ErrorType)
		{
		case EErrorType::NotInPagesList:
			UE_LOGF(LogHAL, Error, "Can't find allocation 0x%llx in the pages list!", ErrorData.Ptr);
			break;
		case EErrorType::InvalidAllocationGroup:
			UE_LOGF(LogHAL, Error, "Allocation 0x%llx has a null AllocationGroup!", ErrorData.Ptr);
			break;
		case EErrorType::InvalidSystemsTag:
			UE_LOGF(LogHAL, Error, "Allocation 0x%llx has a null SystemsTag!", ErrorData.Ptr);
			break;
		case EErrorType::InvalidSize:
			UE_LOGF(LogHAL, Error, "Allocation 0x%llx has negative size!", ErrorData.Ptr);
			break;
		case EErrorType::ExtendedBeyondPages:
			UE_LOGF(LogHAL, Error, "Allocation 0x%llu extended beyond the pages!", ErrorData.Ptr);
			break;
		}
	}

	CountsPerTag.ValueSort([](const FCounts& A, const FCounts& B)
	{
		return A.TotalBytes > B.TotalBytes;
	});

	TArray<FString> Lines;
	Lines.Reserve(CountsPerTag.Num() * 2 + 1);

	Lines.Add(TEXT("Tag,SharedKib,PrivateKib,SplitKib,UnrefKib,TotalKib,SharedCount,PrivateCount,SplitCount,UnrefCount,TotalCount,CrossCount"));


	for (const TPair<const FTagData*, FCounts>& P : CountsPerTag)
	{
		FLowLevelAllocInfo AllocInfoPlaceholder;
		const FTagData* Tag = P.Key;
		Lines.Add(
			FString::Printf(TEXT("%s,%llu,%llu,%llu,%llu,%llu,%llu,%llu,%llu,%llu,%llu,%llu"),
				*Tag->GetDisplayName().ToString(),
				P.Value.ByteCount[Class_Shared] / (1024),
				P.Value.ByteCount[Class_Private] / (1024),
				P.Value.ByteCount[Class_Split] / (1024),
				P.Value.ByteCount[Class_Unreferenced] / (1024),
				P.Value.TotalBytes / (1024),
				P.Value.AllocCount[Class_Shared],
				P.Value.AllocCount[Class_Private],
				P.Value.AllocCount[Class_Split],
				P.Value.AllocCount[Class_Unreferenced],
				P.Value.TotalAllocations,
				P.Value.CrossPageAllocationCount)
			);
	}

	// This should be UE::LLM::GetLLMProfilingDir(), i.e. "Saved/Profiling/LLM", but kept to "Saved/LLM" for legacy reasons.
	FString SavedDir = FPaths::ProjectSavedDir() / TEXT("LLM");

	uint16 ForkId = FForkProcessHelper::GetForkedChildProcessIndex();
	FString ForkString = ForkId == 0 ? FString("Parent") : FString::Printf(TEXT("Child%d"), ForkId);

	const TCHAR* TrackerString = TEXT("Default");
	if (Tracker == ELLMTracker::Platform)
	{
		TrackerString = TEXT("Platform");
	}
	static_assert((int)ELLMTracker::Max == 2, "Add other tracker type strings here");

	FString FileName = FString::Printf(TEXT("LLM_PrivateShared_Tracker%s_%s"), TrackerString, *ForkString);

	FString UniqueFilename = SavedDir / FString::Printf(TEXT("%s.csv"), *FileName);
	uint32 Counter = 1;
	while (IFileManager::Get().FileSize(*UniqueFilename) >= 0)
	{
		UniqueFilename = SavedDir / FString::Printf(TEXT("%s_%d.csv"), *FileName, Counter);
		Counter++;
	}

	if (FFileHelper::SaveStringArrayToFile(Lines, *UniqueFilename) == false)
	{
		UE_LOGF(LogHAL, Error, "Failed to write LLM Private/Shared CSV file: %ls\n", *UniqueFilename);
		return false;
	}

	UE_LOGF(LogHAL, Log, "Wrote LLM Private/Shared CSV file: %ls", *UniqueFilename);
	return true;
#endif // else !UE_ONLY_USE_PLATFORM_TRACKER
}

bool FLLMTracker::DumpTags()
{
	TArray<FString> Lines;

	{
		UE::TSharedLock Lock(LLMRef.TagDataMutex);

		int32 NumTags = LLMRef.TagDatas->Num();
		Lines.Reserve(NumTags + 1);

		Lines.Add(TEXT("Index,Id,Name,DisplayName,Parent,StatName,SummaryStatName,ReferenceSource,EnumTag,bHasEnumTag,TagSet,bIsFinishConstructed,bIsReportable,bIsTraceReportable"));

		for (FTagData* TagData : (*LLMRef.TagDatas))
		{
			Lines.Add(
				FString::Printf(TEXT("%d,0x%llX,%s,%s,%s,%s,%s,%s,%u,%s,%u,%s,%s,%s"),
					TagData->GetIndex(),
					uint64(TagData),
					*TagData->GetName().ToString(),
					*TagData->GetDisplayName().ToString(),
					TagData->IsParentConstructed() ?
						(TagData->GetParent() ? *FString::Printf(TEXT("0x%llX"), uint64(TagData->GetParent())) : TEXT("")) :
						*TagData->GetParentName().ToString(),
					*TagData->GetStatName().ToString(),
					*TagData->GetSummaryStatName().ToString(),
					ToString(TagData->GetReferenceSource()),
					uint32(TagData->GetEnumTag()),
					TagData->HasEnumTag() ? TEXT("true") : TEXT("false"),
					uint32(TagData->GetTagSet()),
					TagData->IsFinishConstructed() ? TEXT("true") : TEXT("false"),
					TagData->IsStatsReportable() ? TEXT("true") : TEXT("false"),
					TagData->IsTraceReportable() ? TEXT("true") : TEXT("false"))
				);
		}
	}

	FString SavedDir = UE::LLM::GetLLMProfilingDir();

	FString FileName("LLM_Tags");

	FString UniqueFilename = SavedDir / FString::Printf(TEXT("%s.csv"), *FileName);
	uint32 Counter = 1;
	while (IFileManager::Get().FileSize(*UniqueFilename) >= 0)
	{
		UniqueFilename = SavedDir / FString::Printf(TEXT("%s_%d.csv"), *FileName, Counter);
		Counter++;
	}

	if (FFileHelper::SaveStringArrayToFile(Lines, *UniqueFilename) == false)
	{
		UE_LOGF(LogHAL, Error, "Failed to write LLM Tags CSV file: %ls\n", *UniqueFilename);
		return false;
	}

	UE_LOGF(LogHAL, Log, "Wrote LLM Tags CSV file: %ls", *UniqueFilename);
	return true;
}

void FLLMTracker::PublishCsv(UE::LLM::ESizeParams SizeParams)
{
	CsvWriter.Publish(LLMRef, TagSizes, OverrideTrackedTotalTagData, OverrideUntaggedTagData, GetTrackedTotal(SizeParams), SizeParams);
}

void FLLMTracker::PublishTrace(UE::LLM::ESizeParams SizeParams)
{
	// Trace tracker does not support the handling of Snapshot currently
	EnumRemoveFlags(SizeParams, UE::LLM::ESizeParams::RelativeToSnapshot);

	TraceWriter.Publish(LLMRef, TagSizes, OverrideTrackedTotalTagData, OverrideUntaggedTagData, GetTrackedTotal(SizeParams), SizeParams);
}

void FLLMTracker::PublishCsvProfiler(UE::LLM::ESizeParams SizeParams)
{
	CsvProfilerWriter.Publish(LLMRef, TagSizes, OverrideTrackedTotalTagData, OverrideUntaggedTagData, GetTrackedTotal(SizeParams), SizeParams);
}

void FLLMTracker::OnTagsResorted(FTagDataArray& OldTagDatas)
{
	// In current code we do not have any persistent sorted list of tags by Index; we create one every update instead.
	// Keep this hook available in case that ever changes.
}

void FLLMTracker::LockAllThreadsSharedData(bool bLock)
{
	if (bLock)
	{
		UpdateThreads();
		PendingThreadStatesGuard.Lock();
	}

	for (FLLMThreadState* ThreadState : ThreadStates)
	{
		ThreadState->LockSharedData(bLock);
	}

	if (!bLock)
	{
		PendingThreadStatesGuard.Unlock();
	}
}

void FLLMTracker::BeginActiveSetsChange(
	TArray<TArray<FAllocationGroupTrackingData, FDefaultLLMAllocator>, FDefaultLLMAllocator>& ThreadStateAllocationGroupTrackingDatas)
{
	ThreadStateAllocationGroupTrackingDatas.SetNum(ThreadStates.Num());
	for (int32 Index = 0; Index < ThreadStates.Num(); ++Index)
	{
		FLLMThreadState* ThreadState = ThreadStates[Index];
		ThreadState->BeginActiveSetsChange(ThreadStateAllocationGroupTrackingDatas[Index]);
	}
}

void FLLMTracker::EndActiveSetsChange(const FActiveTags& NewDefaults,
	TArray<TArray<FAllocationGroupTrackingData, FDefaultLLMAllocator>, FDefaultLLMAllocator>& ThreadStateAllocationGroupTrackingDatas)
{
	ThreadStateAllocationGroupTrackingDatas.SetNum(ThreadStates.Num());
	for (int32 Index = 0; Index < ThreadStates.Num(); ++Index)
	{
		FLLMThreadState* ThreadState = ThreadStates[Index];
		ThreadState->EndActiveSetsChange(NewDefaults, ThreadStateAllocationGroupTrackingDatas[Index]);
	}
}

const FTagData* FLLMTracker::GetActiveTagData(ELLMTagSet TagSet)
{
	FLLMThreadState* State = GetOrCreateState();
	return State->GetTopTag(TagSet);
}

TArray<const FTagData*> FLLMTracker::GetTagDatas(ELLMTagSet TagSet)
{
	TArray<const FTagData*> FoundTagDatas;
	TagSizes.GetKeys(FoundTagDatas);

	return FoundTagDatas.FilterByPredicate([TagSet](const FTagData* InTagData) {
		return InTagData != nullptr && InTagData->GetTagSet() == TagSet;
	});
}

void FLLMTracker::GetTagsNamesWithAmount(TMap<FName, uint64>& OutTagsNamesWithAmount,
	ELLMTagSet TagSet /* = ELLMTagSet::None */)
{
	for (const TPair<const FTagData*, FTrackerTagSizeData>& It : TagSizes)
	{
		const FTagData* TagData = It.Key;
		if (TagData->GetTagSet() == TagSet)
		{
			OutTagsNamesWithAmount.Add(TagData->GetName(), It.Value.GetSize(UE::LLM::ESizeParams::Default));
		}
	}
}

int32 FLLMGlobals::GetIndexInFActiveTags(ELLMTagSet TagSet)
{
	int32 Index = 0;
	for (int32 TagSetIndex = 0; TagSetIndex < static_cast<uint8>(ELLMTagSet::Max); ++TagSetIndex)
	{
		ELLMTagSet IterTagSet = static_cast<ELLMTagSet>(TagSetIndex);
		if (IterTagSet == TagSet)
		{
			if (!ActiveSets[TagSetIndex])
			{
				return -1;
			}
			else
			{
				return Index;
			}
		}
		if (ActiveSets[TagSetIndex])
		{
			++Index;
		}
	}
	return -1;
}

void FLLMTracker::EnumerateAllocations(TFunctionRef<void(void* Ptr, int64 Size, FAllocationGroup* Group)> Callback)
{
#if !UE_ONLY_USE_PLATFORM_TRACKER
	// Callers must call outside of any LLM locks, and not call LLM functions during callback, because this function
	// enters the AllocationsLock, which must be not be simultaneously entered with other LLM locks in incorrect lock
	// ordering.

	// We cannot call GetGroup or read AllocationGroups to find the AllocationGroup used by each AllocInfo when we are
	// within AllocationMap.LockAll(), because reading AllocationGroups or calling GetGroup locks AllocationGroupsIndexMutex
	// and the mutex enter order between AllocationGroupsIndexMutex and the AllocationMap's lock is that
	// AllocationGroupsIndexMutex must be entered before the AllocationMap (that is the order used by e.g. TrackAlloc).
	// So copy AllocationGroups before entering AllocationMap.LockAll().

	// Note that this cache works even when !LLM_ALLOW_NAMES_TAGS, because the AllocationGroup's index when
	// !LLM_ALLOW_NAMES_TAGS is defined to be equal to the ELLMTag, which is the compressed tag.
	TArray<FAllocationGroup*, FDefaultLLMAllocator> AllocationGroupCached;
	{
		UE::TSharedLock Lock(LLMRef.AllocationGroupsIndexMutex);
		AllocationGroupCached = *LLMRef.AllocationGroups;
	}

	AllocationMap.LockAll();
	for (const FLLMAllocMap::FTuple& Tuple : AllocationMap)
	{
		void* Ptr = Tuple.Key.GetPointer();
		int64 SizeLow = static_cast<int64>(Tuple.Value1);
		int64 SizeHigh = Tuple.Key.GetExtraData();
		int64 Size = (SizeHigh << 32ull) | SizeLow;
		const FLLMTracker::FLowLevelAllocInfo& Info = Tuple.Value2;
		FAllocationGroup* AllocationGroup = nullptr;
		int32 CompressedTagIndex = Info.GetCompressedTag();
		if (AllocationGroupCached.IsValidIndex(CompressedTagIndex))
		{
			AllocationGroup = AllocationGroupCached[CompressedTagIndex];
		}
		Callback(Ptr, Size, AllocationGroup);
	}
	AllocationMap.UnlockAll();
#endif // !UE_ONLY_USE_PLATFORM_TRACKER
}

void FLLMTracker::GetTagsNamesWithAmountFiltered(TMap<FName, uint64>& OutTagsNamesWithAmount,
	ELLMTagSet TagSet /* = ELLMTagSet::None */, TArray<FLLMTagSetAllocationFilter>& Filters)
{
	// This function must be called within the UpdateLock, to avoid the possibility of AllocationGroups being deleted
	// while the function is in progress.
#if UE_ONLY_USE_PLATFORM_TRACKER
	return;
#else // !UE_ONLY_USE_PLATFORM_TRACKER

	int32 IndexInActiveTags = LLMRef.GetIndexInFActiveTags(TagSet);
	if (IndexInActiveTags == -1)
	{
		return;
	}

	TArray<int32> IndexOfFiltersInActiveTags;
	IndexOfFiltersInActiveTags.SetNum(Filters.Num());
	for (int32 FilterIndex = 0; FilterIndex < Filters.Num(); ++FilterIndex)
	{
		IndexOfFiltersInActiveTags[FilterIndex] = LLMRef.GetIndexInFActiveTags(Filters[FilterIndex].TagSet);
	}

	TMap<const FTagData*, int64, FDefaultSetLLMAllocator> Results;
	{
		UE::TSharedLock Lock(LLMRef.TagDataMutex);
		Results.Reserve(LLMRef.TagDatas->Num());
	}

	EnumerateAllocations(
		[&Filters, &IndexOfFiltersInActiveTags, IndexInActiveTags, &Results]
		(void* Ptr, int64 Size, FAllocationGroup* AllocationGroup)
		{
			if (!AllocationGroup)
			{
				return;
			}
			if (Size == 0)
			{
				return;
			}

			bool bIncludeAllocation = true;
			for (int32 FilterIndex = 0; FilterIndex < Filters.Num(); ++FilterIndex)
			{
				const FLLMTagSetAllocationFilter& Filter = Filters[FilterIndex];
				int32 IndexOfFilterInActiveTags = IndexOfFiltersInActiveTags[FilterIndex];

				const FTagData* Data = nullptr;
				if (IndexOfFilterInActiveTags != -1)
				{
					Data = AllocationGroup->GetActiveTags()[IndexOfFilterInActiveTags];
				}
				if (!Data || Data->GetName() != Filter.Name)
				{
					bIncludeAllocation = false;
					break;
				}
			}

			if (bIncludeAllocation)
			{
				const FTagData* Data = AllocationGroup->GetActiveTags()[IndexInActiveTags];
				if (Data)
				{
					Results.FindOrAdd(Data) += Size;
				}
			}
		});

	for (const TPair<const FTagData*, int64>& Pair : Results)
	{
		check(Pair.Value != 0);
		OutTagsNamesWithAmount.FindOrAdd(Pair.Key->GetName(), 0) += Pair.Value;
	}
#endif // !UE_ONLY_USE_PLATFORM_TRACKER
}

bool FLLMTracker::FindTagsForPtr(FInPtr InPtr, TArray<const FTagData *, TInlineAllocator<static_cast<int32>(ELLMTagSet::Max)>>& OutTags) const
{
#if !UE_ONLY_USE_PLATFORM_TRACKER
	uint32 Size;
	FLowLevelAllocInfo AllocInfoPtr;
	PointerKey FoundKey = AllocationMap.Find(PointerKey(InPtr), Size, AllocInfoPtr);
	if (!FoundKey)
	{
		return false;
	}
	FAllocationGroup* AllocationGroup = AllocInfoPtr.GetGroup(LLMRef);
	if (!AllocationGroup)
	{
		return false;
	}
	const FActiveTags& ActiveTags = AllocationGroup->GetActiveTags();

	OutTags.SetNumUninitialized(static_cast<int32>(ELLMTagSet::Max));
	int32 IndexInActiveTags = 0;
	for (int32 TagSetAsInteger = 0; TagSetAsInteger < static_cast<int32>(ELLMTagSet::Max); TagSetAsInteger++)
	{
		if (LLMRef.ActiveSets[TagSetAsInteger])
		{
			OutTags[TagSetAsInteger] = ActiveTags[IndexInActiveTags];
			++IndexInActiveTags;
		}
		else
		{
			OutTags[TagSetAsInteger] = nullptr;
		}
	}

	return true;
#else
	return false;
#endif // !UE_ONLY_USE_PLATFORM_TRACKER
}

int64 FLLMTracker::GetTagAmount(const FTagData* TagData, UE::LLM::ESizeParams SizeParams) const
{
	const FTrackerTagSizeData* AllocationData = TagSizes.Find(TagData);
	if (AllocationData)
	{
		return AllocationData->GetSize(SizeParams);
	}
	else
	{
		return 0;
	}
}

void FLLMTracker::SetTagAmountExternal(const FTagData* TagData, int64 Amount, bool bAddToTotal)
{
	FTrackerTagSizeData& AllocationData = TagSizes.FindOrAdd(TagData);
	AllocationData.bExternalValid = true;
	AllocationData.bExternalAddToTotal = bAddToTotal;
	AllocationData.ExternalAmount = Amount;
}

void FLLMTracker::SetTagAmountInUpdate(const FTagData* TagData, int64 Amount, bool bAddToTotal)
{
	FTrackerTagSizeData& AllocationData = TagSizes.FindOrAdd(TagData);
	if (bAddToTotal)
	{
		FPlatformAtomics::InterlockedAdd(&TrackedTotal, Amount - AllocationData.Size);
	}
	AllocationData.Size = Amount;
#if LLM_ENABLED_TRACK_PEAK_MEMORY
	AllocationData.PeakSize = FMath::Max(AllocationData.PeakSize, AllocationData.Size);
#endif
}

int64 FLLMTracker::GetAllocTypeAmount(ELLMAllocType AllocType)
{
	return AllocTypeAmounts[static_cast<int32>(AllocType)];
}

FLLMThreadState::FLLMThreadState()
{
	for (int32 Index = 0; Index < static_cast<int32>(ELLMAllocType::Count); ++Index)
	{
		PausedCounter[Index] = 0;
	}

	// By contract AllocationGroupsMutex must be entered around ClearAllocTypeAmounts, but multithreading contracts
	// of that type are not required in constructors, which complete before the object can be used from multiple
	// threads.
	ClearAllocTypeAmounts();
}

FLLMThreadState::~FLLMThreadState()
{
	Clear();
}

void FLLMThreadState::Clear()
{
	UE::TUniqueLock ScopeLockAllocationGroups(ThreadAllocationGroupsMutex);

	for (int32 TagSetAsInteger = 0; TagSetAsInteger < static_cast<int32>(ELLMTagSet::Max); TagSetAsInteger++)
	{
		TagStack[TagSetAsInteger].Empty();
	}
	for (TPair<FActiveTags, FAllocationGroupTrackingData>& Pair : ActiveTagsToTrackingData)
	{
		Pair.Value.ConditionalReset(true /* bReferenceTrack */);
	}
	ActiveTagsToTrackingData.Empty();
	ClearAllocTypeAmounts();
}

void FLLMThreadState::PushTag(const FTagData* TagData, ELLMTagSet TagSet)
{
	TagStack[static_cast<int32>(TagSet)].Add(TagData);
}

void FLLMThreadState::PopTag(ELLMTagSet TagSet)
{
	LLMCheckf(TagStack[static_cast<int32>(TagSet)].Num() > 0,
		TEXT("Called FLLMThreadState::PopTag without a matching Push (stack was empty on pop)"));
	TagStack[static_cast<int32>(TagSet)].Pop(EAllowShrinking::No);
}

const FTagData* FLLMThreadState::GetTopTag(ELLMTagSet TagSet) const
{
	if (TagStack[static_cast<int32>(TagSet)].Num() == 0)
	{
		return nullptr;
	}

	return TagStack[static_cast<int32>(TagSet)].Last();
}

void FLLMThreadState::BeginActiveSetsChange(
	TArray<FAllocationGroupTrackingData, FDefaultLLMAllocator>& AllocationGroupTrackingDatas)
{
	AllocationGroupTrackingDatas.Empty(ActiveTagsToTrackingData.Num());
	for (TPair<FActiveTags, FAllocationGroupTrackingData>& Pair : ActiveTagsToTrackingData)
	{
		AllocationGroupTrackingDatas.Add(MoveTemp(Pair.Value));
	}
	ActiveTagsToTrackingData.Empty();
}

void FLLMThreadState::EndActiveSetsChange(const FActiveTags& NewDefaults,
	TArray<FAllocationGroupTrackingData, FDefaultLLMAllocator>& AllocationGroupTrackingDatas)
{
	ActiveTagsToTrackingData.Reserve(AllocationGroupTrackingDatas.Num());
	for (FAllocationGroupTrackingData& TrackingData : AllocationGroupTrackingDatas)
	{
		FAllocationGroup* Group = TrackingData.GetGroup();
		ActiveTagsToTrackingData.Add(Group->GetActiveTags(), MoveTemp(TrackingData));
	}
}

FActiveTags FLLMThreadState::GetActiveTags(FLLMGlobals& LLMRef) const
{
	// This function must always be called outside of the ThreadAllocationsGroupRWLock, because we may enter that lock
	// during some helper functions.
	FActiveTags Result;
	uint8 IndexInActiveTags = 0;
	for (uint8 TagSetIndex = 0; TagSetIndex < static_cast<uint8>(ELLMTagSet::Max); ++TagSetIndex)
	{
		ELLMTagSet TagSet = static_cast<ELLMTagSet>(TagSetIndex);
		if (!LLMRef.ActiveSets[TagSetIndex])
		{
			continue;
		}

		const FTagData* TagData = GetTopTag(TagSet);
		if (TagData == nullptr)
		{
			TagData = LLMRef.FindOrAddDefaultTagData(TagSet);
		}

#if !LLM_ALLOW_NAMES_TAGS
		if (TagSet == ELLMTagSet::None)
		{
			// When full tags are disabled, we instead store the top-level enumtag parent of the allocation's tag.
			TagData = TagData->GetContainingEnumTagData();
		}
#endif

		Result[IndexInActiveTags] = TagData;
		++IndexInActiveTags;
	}
	return Result;
}

FAllocationGroupTrackingData& FLLMThreadState::FindOrAddTrackingData(const FActiveTags& ActiveTags, FAllocationGroup* AllocationGroup)
{
	// Caller must hold a WriteScopeLock on ThreadAllocationGroupsMutex
	FAllocationGroupTrackingData& TrackingData = ActiveTagsToTrackingData.FindOrAdd(ActiveTags);
	TrackingData.ConditionalInitialize(AllocationGroup, true /* bReferenceTrack */);
	return TrackingData;
}

void FLLMThreadState::TrackAllocation(const void* Ptr, int64 Size, FLLMGlobals& LLMRef, ELLMTracker Tracker,
	ELLMAllocType AllocType, bool bTrackInMemPro, FAllocationGroup*& OutGroup)
{
	FActiveTags ActiveTags = GetActiveTags(LLMRef);
	{
		UE::TUniqueLock WriteScopeLockThread(ThreadAllocationGroupsMutex);

		AllocTypeAmounts[static_cast<int32>(AllocType)] += Size;

		const FTagData* SystemsTagData = ActiveTags.GetSystemsTagData();
		ELLMTag EnumTag = SystemsTagData ? SystemsTagData->GetContainingEnum() : ELLMTag(0);
		if (Tracker == ELLMTracker::Default)
		{
			FPlatformMemory::OnLowLevelMemory_Alloc(Ptr, static_cast<uint64>(Size), static_cast<uint64>(EnumTag));
		}
#if MEMPRO_ENABLED
		if (FMemProProfiler::IsTrackingTag(EnumTag) && bTrackInMemPro)
		{
			MEMPRO_TRACK_ALLOC(const_cast<void*>(Ptr), static_cast<size_t>(Size));
		}
#endif

		FAllocationGroupTrackingData* TrackingData = ActiveTagsToTrackingData.Find(ActiveTags);
		if (TrackingData)
		{
			TrackingData->TrackAllocOrFree(Size, 1);
			OutGroup = TrackingData->GetGroup();
			return;
		}
	}

	{
		UE::TSharedLock ReadScopeLockGlobal(LLMRef.AllocationGroupsMutex);
		FAllocationGroup** AllocationGroup = LLMRef.ActiveTagsToAllocationGroup->Find(ActiveTags);
		if (AllocationGroup)
		{
			UE::TUniqueLock WriteScopeLockThread(ThreadAllocationGroupsMutex);
			FAllocationGroupTrackingData& TrackingData = FindOrAddTrackingData(ActiveTags, *AllocationGroup);
			TrackingData.TrackAllocOrFree(Size, 1);
			OutGroup = *AllocationGroup;
			return;
		}
	}

	{
		UE::TUniqueLock WriteScopeLockGlobal(LLMRef.AllocationGroupsMutex);
		FAllocationGroup* AllocationGroup = LLMRef.FindOrAddAllocationGroup(ActiveTags);
		{
			UE::TUniqueLock WriteScopeLockThread(ThreadAllocationGroupsMutex);
			FAllocationGroupTrackingData& TrackingData = FindOrAddTrackingData(ActiveTags, AllocationGroup);
			TrackingData.TrackAllocOrFree(Size, 1);
			OutGroup = AllocationGroup;
			return;
		}
	}
}

void FLLMThreadState::TrackFree(const void* Ptr, int64 Size, ELLMTracker Tracker, ELLMAllocType AllocType,
	FAllocationGroup* AllocationGroup, bool bTrackInMemPro)
{
	{
		UE::TUniqueLock WriteScopeLockThread(ThreadAllocationGroupsMutex);

		AllocTypeAmounts[static_cast<int32>(AllocType)] -= Size;

		ELLMTag EnumTag = ELLMTag(0);
		if (AllocationGroup)
		{
			EnumTag = AllocationGroup->GetActiveTags().GetSystemsTagData()->GetContainingEnum();
		}
		if (Tracker == ELLMTracker::Default)
		{
			FPlatformMemory::OnLowLevelMemory_Free(Ptr, static_cast<uint64>(Size), static_cast<uint64>(EnumTag));
		}

#if MEMPRO_ENABLED
		if (FMemProProfiler::IsTrackingTag(EnumTag) && bTrackInMemPro)
		{
			MEMPRO_TRACK_FREE(const_cast<void*>(Ptr));
		}
#endif

		if (!AllocationGroup)
		{
			return;
		}

		const FActiveTags& ActiveTags = AllocationGroup->GetActiveTags();
		FAllocationGroupTrackingData* TrackingData = ActiveTagsToTrackingData.Find(ActiveTags);
		if (TrackingData)
		{
			TrackingData->TrackAllocOrFree(-Size, -1);
			return;
		}
	}

	{
		UE::TUniqueLock WriteScopeLockThread(ThreadAllocationGroupsMutex);
		const FActiveTags& ActiveTags = AllocationGroup->GetActiveTags();
		FAllocationGroupTrackingData& TrackingData = FindOrAddTrackingData(ActiveTags, AllocationGroup);
		TrackingData.TrackAllocOrFree(-Size, -1);
		return;
	}

}

void FLLMThreadState::TrackMemory(int64 Amount, FLLMGlobals& LLMRef, ELLMTracker Tracker, ELLMAllocType AllocType, const FTagData* TagData)
{
#if !LLM_ALLOW_NAMES_TAGS
	// When full tags are disabled, we instead store the top-level enumtag parent of the allocation's tag.
	TagData = TagData ? TagData->GetContainingEnumTagData() : nullptr;
#endif

	// TrackMemory only tracks memory on ELLMTagSet::None; all other tagsets are set to default.
	FActiveTags ActiveTags;
	int32 IndexInActiveTags = 0;
	for (uint8 TagSetIndex = 0; TagSetIndex < static_cast<uint8>(ELLMTagSet::Max); ++TagSetIndex)
	{
		if (LLMRef.ActiveSets[TagSetIndex])
		{
			ELLMTagSet TagSet = static_cast<ELLMTagSet>(TagSetIndex);
			if (TagSet == ELLMTagSet::None)
			{
				ActiveTags[IndexInActiveTags] = TagData;
			}
			else
			{
				ActiveTags[IndexInActiveTags] = LLMRef.FindOrAddDefaultTagData(TagSet);
			}
			++IndexInActiveTags;
		}
	}
	LLMCheck(ActiveTags.GetSystemsTagData() == TagData);

	{
		UE::TUniqueLock WriteScopeLockThread(ThreadAllocationGroupsMutex);

		AllocTypeAmounts[static_cast<int32>(AllocType)] += Amount;

		// TODO: Need to expose TrackMemory to Platform-specific trackers and FMemPropProfiler

		FAllocationGroupTrackingData* TrackingData = ActiveTagsToTrackingData.Find(ActiveTags);
		if (TrackingData)
		{
			TrackingData->TrackAllocOrFree(Amount, 0);
			return;
		}
	}

	{
		UE::TSharedLock ReadScopeLockGlobal(LLMRef.AllocationGroupsMutex);
		FAllocationGroup** AllocationGroup = LLMRef.ActiveTagsToAllocationGroup->Find(ActiveTags);
		if (AllocationGroup)
		{
			UE::TUniqueLock WriteScopeLockThread(ThreadAllocationGroupsMutex);
			FAllocationGroupTrackingData& TrackingData = FindOrAddTrackingData(ActiveTags, *AllocationGroup);
			TrackingData.TrackAllocOrFree(Amount, 0);
			return;
		}
	}

	{
		UE::TUniqueLock WriteScopeLockGlobal(LLMRef.AllocationGroupsMutex);
		FAllocationGroup* AllocationGroup = LLMRef.FindOrAddAllocationGroup(ActiveTags);
		{
			UE::TUniqueLock WriteScopeLockThread(ThreadAllocationGroupsMutex);
			FAllocationGroupTrackingData& TrackingData = FindOrAddTrackingData(ActiveTags, AllocationGroup);
			TrackingData.TrackAllocOrFree(Amount, 0);
			return;
		}
	}
}

void FLLMThreadState::TrackMoved(const void* Dest, const void* Source, int64 Size, ELLMTracker Tracker,
	FAllocationGroup* AllocationGroup)
{
	// Update external memory trackers (ideally would want a proper 'move' option on these).
	// We have to enter the mutex because we are not allowed to dereference AllocationGroup without being in the mutex.
	UE::TUniqueLock WriteScopeLockThread(ThreadAllocationGroupsMutex);

	ELLMTag EnumTag = ELLMTag(0);
	if (AllocationGroup)
	{
		EnumTag = AllocationGroup->GetActiveTags().GetSystemsTagData()->GetContainingEnum();
	}

	if (Tracker == ELLMTracker::Default)
	{
		FPlatformMemory::OnLowLevelMemory_Free(Source, static_cast<uint64>(Size), static_cast<uint64>(EnumTag));
		FPlatformMemory::OnLowLevelMemory_Alloc(Dest, static_cast<uint64>(Size), static_cast<uint64>(EnumTag));
	}

#if MEMPRO_ENABLED
	if (FMemProProfiler::IsTrackingTag(EnumTag))
	{
		MEMPRO_TRACK_FREE(const_cast<void*>(Source));
		MEMPRO_TRACK_ALLOC(const_cast<void*>(Dest), static_cast<size_t>(Size));
	}
#endif
}

void FLLMThreadState::LockSharedData(bool bLock)
{
	if (bLock)
	{
		ThreadAllocationGroupsMutex.Lock();
	}
	else
	{
		ThreadAllocationGroupsMutex.Unlock();
	}
}

void FLLMThreadState::FetchAndClearTagSizes(FMapActiveTagsToAllocationGroupTrackingData& OutDatas,
	int64* OutAllocTypeAmounts, EPruningLevel PruningLevel)
{
	{
		// Move the Size/AllocCount data from the thread-shared container into a local copy, and prune the thread-shared
		// container for any recently unused entries.
		UE::TUniqueLock ScopeLock(ThreadAllocationGroupsMutex);
		for (auto TrackingDataIter(ActiveTagsToTrackingData.CreateIterator()); TrackingDataIter; ++TrackingDataIter)
		{
			FAllocationGroupTrackingData& TrackingData = TrackingDataIter.Value();
			FAllocationGroupTrackingData& OutData = OutDatas.FindOrAdd(TrackingDataIter.Key());
			TrackingData.DetachSizeAndActiveAllocs(OutData, PruningLevel);
			if (PruningLevel > EPruningLevel::None
				&& TrackingData.GetUpdatesSinceLastAllocationChange() >= PruneUpdatesForTrackingData)
			{
				TrackingData.ConditionalReset(true /* bReferenceTrack */);
				TrackingDataIter.RemoveCurrent();
			}
		}
		if (PruningLevel > EPruningLevel::None)
		{
			if (PruningLevel >= EPruningLevel::Max ||
				ActiveTagsToTrackingData.Max() > (int32)((float) ActiveTagsToTrackingData.Num() * PrunePopulationRatioForTrackingData))
			{
				ActiveTagsToTrackingData.Shrink();
			}
		}

		// Also Move the AllocTypeAmounts data
		for (int32 Index = 0; Index < static_cast<int32>(ELLMAllocType::Count); ++Index)
		{
			OutAllocTypeAmounts[Index] += AllocTypeAmounts[Index];
			AllocTypeAmounts[Index] = 0;
		}
	}
}

void SetMemoryStatByFName(FName Name, int64 Amount)
{
	if (Name != NAME_None)
	{
		SET_MEMORY_STAT_FName(Name, Amount);
	}
}

void FLLMThreadState::ClearAllocTypeAmounts()
{
	// Caller is responsible for holding a lock on ThreadAllocationGroupsMutex or global AllocationGroupsMutex.
	for (int32 Index = 0; Index < static_cast<int32>(ELLMAllocType::Count); ++Index)
	{
		AllocTypeAmounts[Index] = 0;
	}
}

// FLLMCsvWriter implementation.
bool CsvWriter::FColumnKey::operator==(const CsvWriter::FColumnKey& Other) const
{
	return SystemsTag == Other.SystemsTag && CodeOrContentTag == Other.CodeOrContentTag;
}

// Updates the Columns tracking CodeOrContent with their System and CodeOrContent tags as well as calculating their sizes.
// Returns true if the columns were updated.
static bool CalculateCodeOrContentColumns(TLLMMap<CsvWriter::FColumnKey, CsvWriter::FColumnData>& Columns, const ELLMTracker& Tracker,
	FLLMGlobals& LLMRef, const FMapActiveTagsToAllocationGroup* ActiveTagsToAllocationGroup, 
	UE::FPlatformSharedMutex& ReadLockMutex)
{
	// When recording ELLMTagSets::CodeOrContent, we write a column for every combination of ELLMTagSet::None
	// and ELLMTagSet::CodeOrContent, rather than a column for every TagData. To find the sizes of the 
	// combinations, we have to iterate over AllocationGroup sizes rather than TagData sizes.
	// One extra bit of complexity: it is possible we are recording other TagSets as well, in which case
	// each AllocationGroup tracks a more specific combination:
	// { ELLMTagSet::None x ELLMTagSet::CodeOrContent x ELLMTagSet::<Other> }
	// To handle that case, we have to sum up the size of every AllocationGroup that matches the 
	// { ELLMTagSet::None x ELLMTagSet::CodeOrContent }
	// combination we are interested in.
	for (TPair<CsvWriter::FColumnKey, CsvWriter::FColumnData>& Pair : Columns)
	{
		Pair.Value.Size = 0;
	}

	int32 IndexOfCodeOrContentInActiveTags = LLMRef.GetIndexInFActiveTags(ELLMTagSet::CodeOrContent);
	// We are recording CodeOrContent, so it should have an index.
	LLMCheck(IndexOfCodeOrContentInActiveTags >= 0);
	bool bUpdatedColumns = false;

	UE::TSharedLock AllocationsReadLock(ReadLockMutex);
	for (const TPair<FActiveTags, FAllocationGroup*>& GroupPair : (*ActiveTagsToAllocationGroup))
	{
		const FActiveTags& ActiveTags = GroupPair.Key;
		const FTagData* SystemsTag = ActiveTags.GetSystemsTagData();
		if (!SystemsTag->IsReportable())
		{
			continue;
		}
		if (SystemsTag->GetDisplayName() == TagName_Untagged)
		{
			// We write the "Untagged" tag in the 2nd column, and always write it, rather than writing it in the
			// column index where we first notice it. That writing is handled by use of OverrideUntaggedTagData in
			// WriteHeader and AddRow.
			continue;
		}
		const FTagData* CodeOrContentTag = ActiveTags[IndexOfCodeOrContentInActiveTags];
		if (!CodeOrContentTag)
		{
			continue;
		}
		int64 CurrentSize = GroupPair.Value->GetSize(Tracker);
		// AllocationGroups includes some groups used by other Trackers and not by this one. We have to check whether
		// the AllocationGroup has a non-zero size in our Tracker before adding it to our output. 
		if (CurrentSize == 0)
		{
			continue;
		}

		CsvWriter::FColumnData& ColumnData = Columns.FindOrAdd(CsvWriter::FColumnKey{ SystemsTag, CodeOrContentTag });
		if (!ColumnData.bInitialized)
		{
			ColumnData.bInitialized = true;
			ColumnData.Index = Columns.Num() - 1;
			bUpdatedColumns = true;
		}
		ColumnData.Size += CurrentSize;
	}

	return bUpdatedColumns;
}

FLLMCsvWriter::FLLMCsvWriter()
	: Archive(nullptr)
	, LastWriteTime(FPlatformTime::Seconds())
	, WriteCount(0)
	, HeaderMaxSize(-1)
	, bRegisteredFlushDelegate(false)
	, bRecordingCodeOrContent(false)
{
}

FLLMCsvWriter::~FLLMCsvWriter()
{
	delete Archive;
}

void FLLMCsvWriter::Clear()
{
	Columns.Empty();
}

void FLLMCsvWriter::OnPreFork()
{
	if (Archive)
	{
		Archive->Flush();
		delete Archive;
		Archive = nullptr;
	}
}

void FLLMCsvWriter::Flush(bool IsOnCrash)
{
	if (Archive)
	{
		if (IsOnCrash)
		{
			FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Flushing LLM CSV on crash\n"));
		}

		Archive->Flush();
	}
}

void FLLMCsvWriter::SetTracker(ELLMTracker InTracker)
{
	Tracker = InTracker;
}

void FLLMCsvWriter::Publish(FLLMGlobals& LLMRef, const FTrackerTagSizeMap& TagSizes,
	const FTagData* OverrideTrackedTotalTagData, const FTagData* OverrideUntaggedTagData, int64 TrackedTotal,
	UE::LLM::ESizeParams SizeParams)
{
	const double Now = FPlatformTime::Seconds();

	if ((LLMRef.bPublishSingleFrame == false) &&
		(Now - LastWriteTime < (double)CVarLLMWriteInterval.GetValueOnAnyThread()))
	{
		return;
	}

	LastWriteTime = Now;

	const bool bCreatedArchive = CreateArchive(LLMRef);
	const bool bColumnsUpdated = UpdateColumns(LLMRef, TagSizes);

	if (bCreatedArchive || bColumnsUpdated)
	{
		// The column names are written at the start of the archive; when they change we seek back to the start of
		// the file and rewrite the column names.
		WriteHeader(LLMRef, OverrideTrackedTotalTagData, OverrideUntaggedTagData);
	}

	AddRow(LLMRef, TagSizes, OverrideTrackedTotalTagData, OverrideUntaggedTagData, TrackedTotal, SizeParams);
}

const TCHAR* FLLMCsvWriter::GetTrackerCsvName(ELLMTracker InTracker)
{
	switch (InTracker)
	{
		case ELLMTracker::Default: return TEXT("LLM");
		case ELLMTracker::Platform: return TEXT("LLMPlatform");
		default: LLMCheck(false); return TEXT("");
	}
}

// Archive is a binary stream, so we can't just serialize an FString using <<.
void FLLMCsvWriter::Write(FStringView Text)
{
	Archive->Serialize(const_cast<ANSICHAR*>(
		StringCast<ANSICHAR>(Text.GetData(), Text.Len()).Get()), Text.Len() * sizeof(ANSICHAR));
}

bool FLLMCsvWriter::CreateArchive(FLLMGlobals& LLMRef)
{
	if (Archive)
	{
		return false;
	}

	// Initialize some variables that require ActiveSets to have been parsed.
	bRecordingCodeOrContent = Tracker == ELLMTracker::Default
		&& LLMRef.IsTagSetRecordingActiveInner(ELLMTagSet::CodeOrContent);

	// Create the csv file.
	FString Directory = UE::LLM::GetLLMProfilingDir();
	IFileManager::Get().MakeDirectory(*Directory, true);

	const TCHAR* TrackerName = GetTrackerCsvName(Tracker);
	const FDateTime FileDate = FDateTime::Now();
#if PLATFORM_DESKTOP
	FString PlatformName = FPlatformProperties::PlatformName();
#else // Use the CPU for consoles so we can differentiate things like different SKUs of a console generation.
	FString PlatformName = FPlatformMisc::GetCPUBrand().TrimStartAndEnd();
#endif
	PlatformName.ReplaceCharInline(' ', '_');
	PlatformName = FPaths::MakeValidFileName(PlatformName);
#if WITH_SERVER_CODE
	FString Filename = FString::Printf(TEXT("%s/%s_Pid%d_%s_%s.csv"), *Directory, TrackerName,
		FPlatformProcess::GetCurrentProcessId(), *FileDate.ToString(), *PlatformName);
#else
	FString Filename = FString::Printf(TEXT("%s/%s_%s_%s.csv"), *Directory, TrackerName, *FileDate.ToString(),
		*PlatformName);
#endif
	Archive = IFileManager::Get().CreateFileWriter(*Filename, FILEWRITE_AllowRead | FILEWRITE_NoFail);
	LLMCheck(Archive);

	// Create space for column titles that are filled in as we get them.
	int32 MaxRequestedSize = CVarLLMHeaderMaxSize.GetValueOnAnyThread();
	if (MaxRequestedSize > 0)
	{
		HeaderMaxSize = MaxRequestedSize;
	}
	else
	{
		int32 CodeOrContentCodeLength = 0;
		int32 CodeOrContentContentLength = 0;
		if (bRecordingCodeOrContent)
		{
			const FTagData* CodeData = LLMRef.FindOrAddCodeOrContentCodeTagData();
			const FTagData* ContentData = LLMRef.FindOrAddCodeOrContentContentTagData();
			CodeOrContentCodeLength = WriteToString<256>(CodeData->GetDisplayName()).Len();
			CodeOrContentContentLength = WriteToString<256>(ContentData->GetDisplayName()).Len();
		}

		HeaderMaxSize = 0;
		constexpr int32 MaxNameStrLen = 256;
		TStringBuilder<MaxNameStrLen> TagNameStr;
		// Find the name size of all reportable TagDatas. This will not include TagDatas that have not yet been created,
		// but it will gather all of the compiled-in ELLMTag and FName tags. We only allow reporting
		// ELLMTagSet::None, and not many dynamic tags exist in ELLMTagSet::None, so this will reserve space for most of
		// them.
		UE::TSharedLock Lock(LLMRef.TagDataMutex);
		for (FTagData* TagData : (*LLMRef.TagDatas))
		{
			if (TagData->IsReportable())
			{
				TagNameStr.Reset();
				// We have to avoid allocating memory in this loop because we hold an LLM lock.
				TagData->GetDisplayPath(TagNameStr, MaxNameStrLen - 1);
				int32 SizeForTag = TagNameStr.Len() + 1;
				if (bRecordingCodeOrContent)
				{
					SizeForTag += SizeForTag + 1 + CodeOrContentCodeLength +
						SizeForTag + 1 + CodeOrContentContentLength;
				}
				HeaderMaxSize += SizeForTag;
			}
		}
	}

	Write(FString::ChrN(HeaderMaxSize, ' '));
	Write(TEXT("\n"));

	return true;
}

bool FLLMCsvWriter::UpdateColumns(FLLMGlobals& LLMRef, const FTrackerTagSizeMap& TagSizes)
{
	bool bUpdated = false;

	if (bRecordingCodeOrContent)
	{
		bUpdated = CalculateCodeOrContentColumns(Columns, Tracker, LLMRef, LLMRef.ActiveTagsToAllocationGroup, LLMRef.AllocationGroupsMutex);
	}

	for (const TPair<const FTagData*, FTrackerTagSizeData>& It : TagSizes)
	{
		const FTagData* SystemsTag = It.Key;
		if (SystemsTag->GetTagSet() != ELLMTagSet::None || !SystemsTag->IsReportable())
		{
			continue;
		}
		if (SystemsTag->GetDisplayName() == TagName_Untagged)
		{
			// We write the "Untagged" tag in the 2nd column, and always write it, rather than writing it in the
			// column index where we first notice it. That writing is handled by use of OverrideUntaggedTagData in
			// WriteHeader and AddRow.
			continue;
		}

		CsvWriter::FColumnData& ColumnData = Columns.FindOrAdd(CsvWriter::FColumnKey{ SystemsTag, nullptr });
		if (!ColumnData.bInitialized)
		{
			bUpdated = true;
			ColumnData.bInitialized = true;
			ColumnData.Index = Columns.Num() - 1;
		}
		// ColumnData.Size is unused when !bRecordingCodeOrContent, because we already have
		// the accumulated value in TagSizes and do not need to accumulate a sum.
	}
	if (bUpdated)
	{
		Columns.ValueSort([](const CsvWriter::FColumnData& A, const CsvWriter::FColumnData& B)
			{
				return A.Index < B.Index;
			});
	}
	return bUpdated;
}

void FLLMCsvWriter::WriteHeader(FLLMGlobals& LLMRef, const FTagData* OverrideTrackedTotalTagData,
	const FTagData* OverrideUntaggedTagData)
{
	int64 OriginalOffset = Archive->Tell();
	Archive->Seek(0);

	TStringBuilder<256> NameBuffer;
	auto WriteColumnName = [this, &NameBuffer](const CsvWriter::FColumnKey& ColumnKey)
	{
		if (!ColumnKey.SystemsTag)
		{
			return;
		}

		NameBuffer.Reset();
		if (ColumnKey.CodeOrContentTag)
		{
			NameBuffer << ColumnKey.CodeOrContentTag->GetDisplayName() << TEXT("/");
		}

		ColumnKey.SystemsTag->AppendDisplayPath(NameBuffer);
		
		NameBuffer << TEXT(",");
		Write(NameBuffer);
	};

	// Specialcase in bRecordingCodeOrContent that makes it identical to the behavior of !bRecordingCodeOrContent:
	// We combine the Code and Content values for ELLMTags::None tags TrackedTotal and Untagged into a single value
	// and write a column for that, rather than making separate Code and Content values. It is not useful for budgets
	// to show the Content-specific value of TrackedTotal and Untagged, so we merge the two columns to save space.
	WriteColumnName(CsvWriter::FColumnKey{ OverrideTrackedTotalTagData, nullptr });
	WriteColumnName(CsvWriter::FColumnKey{ OverrideUntaggedTagData, nullptr });
	for (const TPair<CsvWriter::FColumnKey, CsvWriter::FColumnData>& Pair: Columns)
	{
		WriteColumnName(Pair.Key);
	}

	int64 ColumnTitleTotalSize = Archive->Tell();
	if (ColumnTitleTotalSize >= HeaderMaxSize)
	{
		UE_LOG(LogHAL, Error,
			TEXT("LLM column titles have overflowed, LLM CSM data will be corrupted. Increase CVarLLMHeaderMaxSize > %" INT64_FMT),
			ColumnTitleTotalSize);
	}

	Archive->Seek(OriginalOffset);
}

void FLLMCsvWriter::AddRow(FLLMGlobals& LLMRef, const FTrackerTagSizeMap& TagSizes,
	const FTagData* OverrideTrackedTotalTagData, const FTagData* OverrideUntaggedTagData, int64 TrackedTotal,
	UE::LLM::ESizeParams SizeParams)
{
	TStringBuilder<256> TextBuffer;
	auto WriteValue = [this, &TextBuffer](int64 Value)
	{
		TextBuffer.Reset();
		TextBuffer.Appendf(TEXT("%0.2f,"), (float)Value / 1024.0f / 1024.0f);
		Write(TextBuffer);
	};
	auto WriteTag = [&WriteValue, &TagSizes, SizeParams](const FTagData* TagData)
	{
		if (!TagData)
		{
			WriteValue(0);
		}
		else
		{
			const FTrackerTagSizeData* AllocationData = TagSizes.Find(TagData);
			if (!AllocationData)
			{
				WriteValue(0);
			}
			else
			{
				WriteValue(AllocationData->GetSize(SizeParams));
			}
		}
	};

	if (OverrideTrackedTotalTagData)
	{
		WriteValue(TrackedTotal);
	}
	if (OverrideUntaggedTagData)
	{
		// All trackers store Untagged tagsizes under the "Untagged" tag.
		// But for some not-yet-investigated legacy reason, when writing CSV we write out the name of a different ELLMTag:
		// ELLMTag::UntaggedTotal or ELLMTag::PlatformUntaggedTotal.
		// So do not read TagSizes[OverrideUntaggedTagData], read TagSizes[ELLMTag::Untagged].
		const FTagData* TagToReadFromForUntagged = LLMRef.FindTagData(TagName_Untagged, ELLMTagSet::None);
		WriteTag(TagToReadFromForUntagged);
	}

	for (TPair<CsvWriter::FColumnKey, CsvWriter::FColumnData>& Pair : Columns)
	{
		if (Pair.Key.CodeOrContentTag)
		{
			WriteValue(Pair.Value.Size);
		}
		else
		{
			WriteTag(Pair.Key.SystemsTag);
		}
	}
	Write(TEXTVIEW("\n"));

	WriteCount++;

	if (CVarLLMWriteInterval.GetValueOnAnyThread())
	{
		UE_LOGF(LogHAL, Log, "Wrote LLM csv line %d", WriteCount);
	}

	if (CVarLLMCsvFlushEveryRow.GetValueOnAnyThread())
	{
		Archive->Flush();
	}
	else if (!bRegisteredFlushDelegate)
	{
		// If we're not flushing every row, lazily register flush delegates to ensure we flush on a crash
		// Note: we intentionally leak this since we can't clean it up safely
		FCoreDelegates::OnHandleSystemError.AddRaw(this, &FLLMCsvWriter::Flush, true);
		FCoreDelegates::GetOutOfMemoryDelegate().AddRaw(this, &FLLMCsvWriter::Flush, true);
		FCoreDelegates::ApplicationWillEnterBackgroundDelegate.AddRaw(this, &FLLMCsvWriter::Flush, false); // When you close the app on some platforms it is suspended rather than terminated
		FCoreDelegates::GetApplicationWillTerminateDelegate().AddRaw(this, &FLLMCsvWriter::Flush, false);
		bRegisteredFlushDelegate = true;
	}

}

// FLLMTraceWriter implementation.
FLLMTraceWriter::FLLMTraceWriter()
{
}

inline void FLLMTraceWriter::SetTracker(ELLMTracker InTracker)
{
	Tracker = InTracker;
}

void FLLMTraceWriter::Clear()
{
	DeclaredTags.Empty();
}

const void* FLLMTraceWriter::GetTagId(const FTagData* TagData)
{
	if (!TagData)
	{
		return nullptr;
	}
	return reinterpret_cast<const void*>(TagData);
}

void FLLMTraceWriter::Publish(FLLMGlobals& LLMRef, const FTrackerTagSizeMap& TagSizes,
	const FTagData* OverrideTrackedTotalTagData, const FTagData* OverrideUntaggedTagData, int64 TrackedTotal,
	UE::LLM::ESizeParams SizeParams)
{
	if (!UE_TRACE_CHANNELEXPR_IS_ENABLED(MemTagChannel))
	{
		return;
	}

	if (!bTrackerSpecSent)
	{
		bTrackerSpecSent = true;
		FAnsiStringView TrackerNames[] = { "Platform", "Default" };
		static_assert((int)ELLMTracker::Platform == 0, "");
		static_assert((int)ELLMTracker::Default == 1, "");
		static_assert(UE_ARRAY_COUNT(TrackerNames) == (int)ELLMTracker::Max, "");
		FAnsiStringView TrackerName = TrackerNames[(uint8)Tracker];
		UE_TRACE_LOG(LLM, TrackerSpec, MemTagChannel, TrackerName.Len() * sizeof(ANSICHAR))
			<< TrackerSpec.TrackerId((uint8)Tracker)
			<< TrackerSpec.Name(TrackerName.GetData(), TrackerName.Len());
	}

	if (!bTagSetSpecSent && Tracker == ELLMTracker::Default)
	{
		bTagSetSpecSent = true;
		FAnsiStringView TagSetNames[] = {
			"System",
			"Assets",
			"AssetClasses",
			"UObjectClasses",
			"CodeOrContent",
		};
		static_assert((int)ELLMTagSet::Assets == 1, "");
		static_assert((int)ELLMTagSet::AssetClasses == 2, "");
		static_assert((int)ELLMTagSet::UObjectClasses == 3, "");
		static_assert((int)ELLMTagSet::CodeOrContent == 4, "");
		static_assert(UE_ARRAY_COUNT(TagSetNames) == (int)ELLMTagSet::Max, "");
		for (uint8 TagSet = 0; TagSet < (uint8)ELLMTagSet::Max; ++TagSet)
		{
			if (LLMRef.IsTagSetRecordingActiveInner((ELLMTagSet)TagSet))
			{
				FAnsiStringView TagSetName = TagSetNames[TagSet];
				UE_TRACE_LOG(LLM, TagSetSpec, MemTagChannel, TagSetName.Len() * sizeof(ANSICHAR))
					<< TagSetSpec.TagSetId(TagSet)
					<< TagSetSpec.Name(TagSetName.GetData(), TagSetName.Len());
			}
		}
	}

	if (OverrideTrackedTotalTagData && !OverrideTrackedTotalTagData->IsTraceReportable())
	{
		OverrideTrackedTotalTagData = nullptr;
	}
	if (OverrideUntaggedTagData && !OverrideUntaggedTagData->IsTraceReportable())
	{
		OverrideUntaggedTagData = nullptr;
	}

	if (OverrideTrackedTotalTagData)
	{
		SendTagDeclaration(OverrideTrackedTotalTagData);
	}
	if (OverrideUntaggedTagData)
	{
		SendTagDeclaration(OverrideUntaggedTagData);
	}
	for (const TPair<const FTagData*, FTrackerTagSizeData>& It : TagSizes)
	{
		const FTagData* TagData = It.Key;
		if (!TagData->IsTraceReportable() || !LLMRef.IsTagSetRecordingActiveInner(TagData->GetTagSet()))
		{
			continue;
		}
		if (OverrideUntaggedTagData &&
			TagData->GetTagSet() == ELLMTagSet::None &&
			TagData->GetName() == TagName_Untagged)
		{
			continue; // Handled by OverrideUntaggedTagData.
		}
		SendTagDeclaration(TagData);
	}

	TArray<uint64, FDefaultLLMAllocator> TagIds;
	TArray<int64, FDefaultLLMAllocator> TagValues;
	TagIds.Reserve(TagSizes.Num() + 2);
	TagValues.Reserve(TagSizes.Num() + 2);
	auto AddValue = [&TagIds, &TagValues](const FTagData* TagData, int64 Value)
	{
		const uint64 TagId = uint64(GetTagId(TagData));
		TagIds.Add(TagId);
		TagValues.Add(Value);
	};

	if (OverrideTrackedTotalTagData)
	{
		AddValue(OverrideTrackedTotalTagData, TrackedTotal);
	}
	if (OverrideUntaggedTagData)
	{
		const FTagData* TagData = LLMRef.FindTagData(TagName_Untagged, ELLMTagSet::None);
		if (!TagData)
		{
			AddValue(OverrideUntaggedTagData, 0);
		}
		else
		{
			const FTrackerTagSizeData* AllocationData = TagSizes.Find(TagData);
			AddValue(OverrideUntaggedTagData, AllocationData ? AllocationData->GetSize(SizeParams) : 0);
		}
	}

	for (const TPair<const FTagData*, FTrackerTagSizeData>& It : TagSizes)
	{
		const FTagData* TagData = It.Key;
		if (!TagData->IsTraceReportable() || !LLMRef.IsTagSetRecordingActiveInner(TagData->GetTagSet()))
		{
			continue;
		}
		if (OverrideUntaggedTagData &&
			TagData->GetTagSet() == ELLMTagSet::None &&
			TagData->GetName() == TagName_Untagged)
		{
			continue; // Handled by OverrideUntaggedTagData.
		}
		AddValue(TagData, It.Value.GetSize(SizeParams));
	}

	LLMCheck(TagIds.Num() == TagValues.Num());
	const uint64 Cycle = FPlatformTime::Cycles64();
	UE_TRACE_LOG(LLM, TagValue, MemTagChannel)
		<< TagValue.TrackerId((uint8)Tracker)
		<< TagValue.Cycle(Cycle)
		<< TagValue.Tags(TagIds.GetData(), TagIds.Num())
		<< TagValue.Values(TagValues.GetData(), TagValues.Num());
}

void FLLMTraceWriter::SendTagDeclaration(const FTagData* TagData)
{
	LLMCheck(TagData);

	bool bAlreadyInSet;
	DeclaredTags.Add(TagData, &bAlreadyInSet);
	if (bAlreadyInSet)
	{
		return;
	}

	const FTagData* Parent = TagData->GetParent();
	if (Parent)
	{
		SendTagDeclaration(Parent);
	}

	TStringBuilder<1024> NameBuffer;
	TagData->AppendDisplayPath(NameBuffer);
	UE_TRACE_LOG(LLM, TagsSpec, MemTagChannel, NameBuffer.Len() * sizeof(ANSICHAR))
		<< TagsSpec.TagId(uint64(GetTagId(TagData)))
		<< TagsSpec.ParentId(uint64(GetTagId(Parent)))
		<< TagsSpec.TagSetId(static_cast<uint8>(TagData->GetTagSet()))
		<< TagsSpec.Name(*NameBuffer, NameBuffer.Len());
};

// FLLMCsvProfilerWriter implementation.
FLLMCsvProfilerWriter::FLLMCsvProfilerWriter()
{
}

void FLLMCsvProfilerWriter::Clear()
{
#if LLM_CSV_PROFILER_WRITER_ENABLED
	ColumnKeyDataToCsvStatName.Empty();
#endif
}

void FLLMCsvProfilerWriter::SetTracker(ELLMTracker InTracker)
{
#if LLM_CSV_PROFILER_WRITER_ENABLED
	check(InTracker == ELLMTracker::Platform || InTracker == ELLMTracker::Default);
	Tracker = InTracker;
#endif
}

void FLLMCsvProfilerWriter::Publish(FLLMGlobals& LLMRef, const FTrackerTagSizeMap& TagSizes,
	const FTagData* OverrideTrackedTotalTagData, const FTagData* OverrideUntaggedTagData, int64 TrackedTotal,
	UE::LLM::ESizeParams SizeParams)
{
#if LLM_CSV_PROFILER_WRITER_ENABLED
	int32 CsvCategoryIndex = (Tracker == ELLMTracker::Platform) ?
		CSV_CATEGORY_INDEX(LLMPlatform) : CSV_CATEGORY_INDEX(LLM);

	if (OverrideTrackedTotalTagData)
	{
		RecordTagToCsv(CsvCategoryIndex, CsvWriter::FColumnKey{OverrideTrackedTotalTagData, nullptr}, TrackedTotal);
	}

	if (OverrideUntaggedTagData)
	{
		const FTagData* TagData = LLMRef.FindTagData(TagName_Untagged, ELLMTagSet::None);
		const FTrackerTagSizeData* AllocationData = TagData ? TagSizes.Find(TagData) : nullptr;
		RecordTagToCsv(CsvCategoryIndex, CsvWriter::FColumnKey{OverrideUntaggedTagData, nullptr},
			AllocationData ? AllocationData->GetSize(SizeParams) : 0);
	}

	// Initialize some variables that require ActiveSets to have been parsed.
	bool bRecordingCodeOrContent = Tracker == ELLMTracker::Default 
		&& LLMRef.IsTagSetRecordingActiveInner(ELLMTagSet::CodeOrContent);

	if (bRecordingCodeOrContent)
	{
		bool bUpdated = CalculateCodeOrContentColumns(Columns, Tracker, LLMRef, LLMRef.ActiveTagsToAllocationGroup, LLMRef.AllocationGroupsMutex);
		if (bUpdated)
		{
			Columns.ValueSort([](const CsvWriter::FColumnData& A, const CsvWriter::FColumnData& B)
				{
					return A.Index < B.Index;
				});
		}

		for (const TPair<CsvWriter::FColumnKey, CsvWriter::FColumnData>& KVPair : Columns)
		{
			RecordTagToCsv(CSV_CATEGORY_INDEX(LLMCodeOrContent), KVPair.Key, KVPair.Value.Size);
		}
	}

	for (const TPair<const FTagData*, FTrackerTagSizeData>& It : TagSizes)
	{
		const FTagData* TagData = It.Key;
		if (!TagData->IsReportable())
		{
			continue;
		}
		if (OverrideUntaggedTagData && TagData->GetName() == TagName_Untagged)
		{
			// Handled separately by OverrideUntaggedTagData.
			continue;
		}
		const FTrackerTagSizeData* AllocationData = TagSizes.Find(TagData);
		RecordTagToCsv(CsvCategoryIndex, CsvWriter::FColumnKey{TagData, nullptr}, AllocationData ? AllocationData->GetSize(SizeParams) : 0);
	}
#endif
}

void FLLMCsvProfilerWriter::RecordTagToCsv(int32 CsvCategoryIndex, const CsvWriter::FColumnKey& ColumnKey, int64 Size)
{
#if LLM_CSV_PROFILER_WRITER_ENABLED
	ensureMsgf(ColumnKey.SystemsTag != nullptr, TEXT("Systems Tag should not be nullptr, check usage."));

	if (ColumnKey.SystemsTag == nullptr)
	{
		return;
	}

	FName NewCsvStatName;
	FName* CsvStatNamePtr = ColumnKeyDataToCsvStatName.Find(ColumnKey);
	if (CsvStatNamePtr == nullptr)
	{
		TStringBuilder<FName::StringBufferSize> DisplayPath;

		// If there is a CodeOrContent tag, prepend to the name
		if (ColumnKey.CodeOrContentTag != nullptr)
		{
			DisplayPath << ColumnKey.CodeOrContentTag->GetDisplayName() << TEXT("/");
		}

		ColumnKey.SystemsTag->AppendDisplayPath(DisplayPath);
		NewCsvStatName = FName(DisplayPath);
		ColumnKeyDataToCsvStatName.Add(ColumnKey, NewCsvStatName);
		CsvStatNamePtr = &NewCsvStatName;
	}
	FCsvProfiler::RecordCustomStat(*CsvStatNamePtr, CsvCategoryIndex, (float)((double)Size / (1024.0 * 1024.0)),
		ECsvCustomStatOp::Set);
#endif
}

} // namespace UE::LLMPrivate

#else // #if ENABLE_LOW_LEVEL_MEM_TRACKER

// We need to stub some functions so things link when the UE_ENABLE_ARRAY_SLACK_TRACKING debug feature is enabled in builds
// where LLM is disabled.  Compiling out the slack tracking code completely is difficult due to include order issues.
// Slack tracking is in a header that must be included before LLM, so it can't access the ENABLE_LOW_LEVEL_MEM_TRACKER
// define.  And moving that define leads to a chain reaction of other include order issues.  It's just not worth it for
// a rarely enabled debug feature to go to all that trouble, when stubbing functions works fine...
#if UE_ENABLE_ARRAY_SLACK_TRACKING
uint8 LlmGetActiveTag() { return 0; }
void ArraySlackTrackInit() {}
void ArraySlackTrackGenerateReport(const TCHAR* Cmd, FOutputDevice& Ar) {}
void FArraySlackTrackingHeader::AddAllocation() {}
void FArraySlackTrackingHeader::RemoveAllocation() {}
void FArraySlackTrackingHeader::UpdateNumUsed(int64 NewNumUsed) {}

FORCENOINLINE void* FArraySlackTrackingHeader::Realloc(void* Ptr, int64 Count, uint64 ElemSize, int32 Alignment)
{
	return FMemory::Realloc(Ptr, Count * ElemSize, Alignment);
}
#endif

#endif  // #else .. #if ENABLE_LOW_LEVEL_MEM_TRACKER
