// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/LowLevelMemTracker.h"

#if ENABLE_LOW_LEVEL_MEM_TRACKER
#include "HAL/PlatformMutex.h"

namespace UE::LLMPrivate
{

namespace AllocatorPrivate
{
	struct FBin;
	struct FPage;
}
/**
 * The allocator LLM uses to allocate internal memory. Uses platform defined
 * allocation functions to grab memory directly from the OS.
 */
class FLLMAllocator
{
public:
	FLLMAllocator();
	~FLLMAllocator();

	static FLLMAllocator*& Get();

	void Initialize(LLMAllocFunction InAlloc, LLMFreeFunction InFree, int32 InPageSize);
	void Clear();
	void* Alloc(size_t Size);
	void* Malloc(size_t Size);
	void Free(void* Ptr, size_t Size);
	void* Realloc(void* Ptr, size_t OldSize, size_t NewSize);
	int64 GetTotal() const;

	template <typename T, typename... ArgsType>
	T* New(ArgsType&&... Args)
	{
		T* Ptr = reinterpret_cast<T*>(Alloc(sizeof(T)));
		new (Ptr) T(Forward<ArgsType>(Args)...);
		return Ptr;
	}

	template <typename T>
	void Delete(T* Ptr)
	{
		if (Ptr)
		{
			Ptr->~T();
			Free(Ptr, sizeof(T));
		}
	}

	LLMAllocFunction GetPlatformAlloc()
	{
		return PlatformAlloc;
	}
	LLMFreeFunction GetPlatformFree()
	{
		return PlatformFree;
	}

private:
	void* AllocPages(size_t Size);
	void FreePages(void* Ptr, size_t Size);
	int32 GetBinIndex(size_t Size) const;

	UE::FPlatformRecursiveMutex Mutex;
	LLMAllocFunction PlatformAlloc;
	LLMFreeFunction PlatformFree;
	AllocatorPrivate::FBin* Bins;
	int64 Total;
	int32 PageSize;
	int32 NumBins;

	friend struct UE::LLMPrivate::AllocatorPrivate::FPage;
	friend struct UE::LLMPrivate::AllocatorPrivate::FBin;
};

} // namespace UE::LLMPrivate

#endif		// #if ENABLE_LOW_LEVEL_MEM_TRACKER
