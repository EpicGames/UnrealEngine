// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/LowLevelMemTracker.h"

#if ENABLE_LOW_LEVEL_MEM_TRACKER
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/SortedMap.h"
#include "HAL/CriticalSection.h"
#include "HAL/LLM/AllocationGroup.h"
#include "HAL/LLM/LLMPrivateDefines.h"
#include "HAL/LLM/MemoryUtils.h"
#include "HAL/PlatformMutex.h"
#include "Misc/Optional.h"
#include "Misc/ScopeRWLock.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "Templates/Function.h"

void GlobalRegisterTagDeclaration(FLLMTagDeclaration& TagDeclaration);

namespace UE::LLMPrivate
{

class FTagData;

namespace CsvWriter
{

struct FColumnKey
{
	const FTagData* SystemsTag = nullptr;
	const FTagData* CodeOrContentTag = nullptr;
	bool operator==(const FColumnKey& Other) const;

	friend uint32 GetTypeHash(const FColumnKey& Key)
	{
		constexpr uint32 HashPrime = 101;
		return ::GetTypeHash(Key.SystemsTag) * HashPrime + ::GetTypeHash(Key.CodeOrContentTag);
	}
};

struct FColumnData
{
	int64 Size = 0;
	int32 Index = 0;
	bool bInitialized = false;
};

} // namespace UE::LLMPrivate::CsvWriter

struct FTagDataNameKey
{
	FName Name;
	ELLMTagSet TagSet;
		
	FTagDataNameKey(FName InName, ELLMTagSet InTagSet) :
		Name(InName),
		TagSet(InTagSet)
	{
	}

	friend uint32 GetTypeHash(const FTagDataNameKey& Key)
	{
		constexpr uint32 HashPrime = 101;
		return GetTypeHash(Key.Name) + HashPrime * static_cast<uint32>(Key.TagSet);
	}

	friend bool operator==(const FTagDataNameKey& A, const FTagDataNameKey& B)
	{
		return A.Name == B.Name && A.TagSet == B.TagSet;
	}

	friend bool operator!=(const FTagDataNameKey& A, const FTagDataNameKey& B)
	{
		return (A.Name != B.Name || A.TagSet != B.TagSet);
	}

	FTagDataNameKey() = delete;
};

/**
 * FTagData: Description of the properties of a Tag that can be used in LLM_SCOPE
 */
class FTagData
{
public:

	FTagData(FName InName, ELLMTagSet InTagSet, FName InDisplayName, FName InParentName, FName InStatName,
		FName InSummaryStatName, bool bInHasEnumTag, ELLMTag InEnumTag, ETagReferenceSource InReferenceSource);
	FTagData(FName InName, ELLMTagSet InTagSet, FName InDisplayName, const FTagData* InParent, FName InStatName,
		FName InSummaryStatName, bool bInHasEnumTag, ELLMTag InEnumTag, ETagReferenceSource InReferenceSource);
	~FTagData();

	bool IsParentConstructed() const;
	bool IsFinishConstructed() const;
	FName GetName() const;
	FName GetDisplayName() const;
	void GetDisplayPath(FStringBuilderBase& Result, int32 MaxLen=-1) const;
	void AppendDisplayPath(FStringBuilderBase& Result, int32 MaxLen=-1) const;
	const FTagData* GetParent() const;
	FName GetParentName() const;
	FName GetParentNameSafeBeforeFinishConstruct() const;
	FName GetStatName() const;
	FName GetSummaryStatName() const;
	ELLMTag GetEnumTag() const;
	ELLMTagSet GetTagSet() const;
	bool HasEnumTag() const;
	const FTagData* GetContainingEnumTagData() const;
	ELLMTag GetContainingEnum() const;
	ETagReferenceSource GetReferenceSource() const;
	int32 GetIndex() const;
	bool IsReportable() const;
	bool IsStatsReportable() const;
	bool IsTraceReportable() const;

	void SetParent(const FTagData* InParent);
	void SetIndex(int32 InIndex);
	void SetIsReportable(bool bInReportable);
		void SetIsTraceReportable(bool bInTraceReportable);
	void SetFinishConstructed();

	// These functions are normally invalid - these properties should be immutable - but are called for EnumTags during bootstrapping
	FTagData(ELLMTag InEnumTag);
	void SetName(FName InName);
	void SetDisplayName(FName InDisplayName);
	void SetStatName(FName InStatName);
	void SetSummaryStatName(FName InSummaryStatName);
	void SetParentName(FName InParentName);

private:
	bool IsUsedAsDisplayParent() const;

	FName Name;
	FName DisplayName;
	union
	{
		const FTagData* Parent;
		FName ParentName;
	};
	FName StatName;
	FName SummaryStatName;
	int32 Index;
	ELLMTag EnumTag;
	ETagReferenceSource ReferenceSource;
	ELLMTagSet TagSet;
	bool bIsFinishConstructed;
	bool bParentIsName;
	bool bHasEnumTag;
	bool bIsReportable;
	bool bIsTraceReportable;
};

// TagData container types that use FLLMAllocator
class FTagDataNameMap : public TMap<FTagDataNameKey, FTagData*, FDefaultSetLLMAllocator>
{
	using TMap<FTagDataNameKey, FTagData*, FDefaultSetLLMAllocator>::TMap;
};
class FConstTagDataArray : public TArray<const FTagData*, FDefaultLLMAllocator>
{
	using TArray<const FTagData*, FDefaultLLMAllocator>::TArray;
};
class FTagDataArray : public TArray<FTagData*, FDefaultLLMAllocator>
{
	using TArray<FTagData*, FDefaultLLMAllocator>::TArray;
};

class FTagFilter : public TLLMSet<FName>
{
};

} // namespace UE::LLMPrivate

// Interface for Algo::TopologicalSort for FTagDataArray
inline UE::LLMPrivate::FTagData** GetData(UE::LLMPrivate::FTagDataArray& Array)
{
	return Array.GetData();
}
inline UE::LLMPrivate::FTagDataArray::SizeType GetNum(UE::LLMPrivate::FTagDataArray& Array)
{
	return Array.Num();
}

namespace UE::LLMPrivate
{

/** Size information stored on the tracker for a tag; includes amounts aggregated from threadstates and from external api users */
struct FTrackerTagSizeData
{
	int64 Size = 0;
#if LLM_ENABLED_TRACK_PEAK_MEMORY
	int64 PeakSize = 0;
#endif
	int64 SizeInSnapshot = 0;
	int64 ExternalAmount = 0;
	bool bExternalValid = false;
	bool bExternalAddToTotal = false;

	int64 GetSize(UE::LLM::ESizeParams SizeParams) const
	{
		int64 CurrentSize = Size;

#if LLM_ENABLED_TRACK_PEAK_MEMORY
		if (EnumHasAnyFlags(SizeParams, UE::LLM::ESizeParams::ReportPeak))
		{
			CurrentSize = PeakSize;
		}
#endif
		// Note, this will also subtract the snapshotted size from PeakSize if that flag is enabled
		if (EnumHasAnyFlags(SizeParams, UE::LLM::ESizeParams::RelativeToSnapshot))
		{
			CurrentSize = FMath::Clamp<int64>(CurrentSize - SizeInSnapshot, 0, INT64_MAX); //-V569
		}

		return CurrentSize;
	};

	void CaptureSnapshot()
	{
		SizeInSnapshot = Size;
	}

	void ClearSnapshot()
	{
		SizeInSnapshot = 0;
	}
};
typedef TLLMMap<const FTagData*, FTrackerTagSizeData> FTrackerTagSizeMap;

/**
 * Size information stored on each threadstate for a tag
 * TagSizes are sorted by Index instead of by pointer in the ThreadTagSizeMap to enforce the constraint that Parents come before children
 */
struct FThreadTagSizeData
{
	const FTagData* TagData = nullptr;
	int64 Size = 0;
};
typedef TSortedMap<int32, FThreadTagSizeData, FDefaultLLMAllocator> FThreadTagSizeMap;

} // namespace UE::LLMPrivate

FName LLMGetTagUniqueName(ELLMTag Tag);

namespace UE::LLMPrivate
{

#if !PLATFORM_HAS_MULTITHREADED_PREMAIN

struct FEnableStateScopeLock
{
};

#else // !PLATFORM_HAS_MULTITHREADED_PREMAIN

struct FEnableStateScopeLock
{
	// Undefine copy/move constructor since FReadScopeLock does not support it.
	FEnableStateScopeLock() = default;
	FEnableStateScopeLock(const FEnableStateScopeLock&) = delete;
	FEnableStateScopeLock(FEnableStateScopeLock&&) = delete;

	TOptional<FReadScopeLock> Inner;
};

#endif // else !PLATFORM_HAS_MULTITHREADED_PREMAIN

/** FLLMCsvWriter: class for writing out the LLM tag sizes to a csv file every few seconds. */
class FLLMCsvWriter
{
public:
	FLLMCsvWriter();
	~FLLMCsvWriter();

	void SetTracker(ELLMTracker InTracker);
	void Clear();

	void Publish(FLLMGlobals& LLMRef, const FTrackerTagSizeMap& TagSizes,
		const FTagData* OverrideTrackedTotalTagData, const FTagData* OverrideUntaggedTagData, int64 TrackedTotal,
		UE::LLM::ESizeParams SizeParams);

	void OnPreFork();

	void Flush(bool IsOnCrash);

private:
	void Write(FStringView Text);
	static const TCHAR* GetTrackerCsvName(ELLMTracker InTracker);

	bool CreateArchive(FLLMGlobals& LLMRef);
	bool UpdateColumns(FLLMGlobals& LLMRef, const FTrackerTagSizeMap& TagSizes);
	void WriteHeader(FLLMGlobals& LLMRef, const FTagData* OverrideTrackedTotalTagData, const FTagData* OverrideUntaggedTagData);
	void AddRow(FLLMGlobals& LLMRef, const FTrackerTagSizeMap& TagSizes,
		const FTagData* OverrideTrackedTotalName, const FTagData* OverrideUntaggedName, int64 TrackedTotal,
		UE::LLM::ESizeParams SizeParams);

	TLLMMap<CsvWriter::FColumnKey, CsvWriter::FColumnData> Columns;
	FArchive* Archive;
	double LastWriteTime;
	int32 WriteCount;
	int32 HeaderMaxSize;
	ELLMTracker Tracker;
	bool bRegisteredFlushDelegate;
	bool bRecordingCodeOrContent;
};

/** Outputs the LLM tags and sizes to TraceLog events. */
class FLLMTraceWriter
{
public:
	FLLMTraceWriter();
	void SetTracker(ELLMTracker InTracker);
	void Clear();
	void Publish(FLLMGlobals& LLMRef, const FTrackerTagSizeMap& TagSizes,
		const FTagData* OverrideTrackedTotalName, const FTagData* OverrideUntaggedName, int64 TrackedTotal,
		UE::LLM::ESizeParams SizeParams);

private:
	static const void* GetTagId(const FTagData* TagData);
	void SendTagDeclaration(const FTagData* TagData);

	ELLMTracker Tracker;
	TLLMSet<const FTagData*> DeclaredTags;
	bool bTrackerSpecSent = false;
	bool bTagSetSpecSent = false;
};

/** FLLMCsvWriter: class for writing out LLM stats to the Csv Profiler. */
class FLLMCsvProfilerWriter
{
public:
	FLLMCsvProfilerWriter();
	void Clear();
	void SetTracker(ELLMTracker InTracker);
	void Publish(FLLMGlobals& LLMRef, const FTrackerTagSizeMap& TagSizes,
		const FTagData* OverrideTrackedTotalName, const FTagData* OverrideUntaggedName, int64 TrackedTotal,
		UE::LLM::ESizeParams SizeParams);
protected:
	void RecordTagToCsv(int32 CsvCategoryIndex, const CsvWriter::FColumnKey& ColumnKey, int64 Size);
#if LLM_CSV_PROFILER_WRITER_ENABLED
	ELLMTracker Tracker;
	TLLMMap<CsvWriter::FColumnKey, FName> ColumnKeyDataToCsvStatName;
#endif
private:
	TLLMMap<CsvWriter::FColumnKey, CsvWriter::FColumnData> Columns;
};

/** Per-thread state in an LLMTracker. */
class FLLMThreadState
{
public:
	FLLMThreadState();
	~FLLMThreadState();

	void Clear();

	void PushTag(const FTagData* TagData, ELLMTagSet TagSet);
	void PopTag(ELLMTagSet TagSet);
	const FTagData* GetTopTag(ELLMTagSet TagSet) const;
	void TrackAllocation(const void* Ptr, int64 Size, FLLMGlobals& LLMRef, ELLMTracker Tracker,
		ELLMAllocType AllocType, bool bTrackInMemPro, FAllocationGroup*& OutGroup);
	FAllocationGroupTrackingData& FindOrAddTrackingData(const FActiveTags& ActiveTags,
		FAllocationGroup* AllocationGroup);
	void TrackFree(const void* Ptr, int64 Size, ELLMTracker Tracker, ELLMAllocType AllocType,
		FAllocationGroup* AllocationGroup, bool bTrackInMemPro);
	void TrackMemory(int64 Amount, FLLMGlobals& LLMRef, ELLMTracker Tracker, ELLMAllocType AllocType,
		const FTagData* TagData);
	void TrackMoved(const void* Dest, const void* Source, int64 Size, ELLMTracker Tracker,
		FAllocationGroup* AllocationGroup);

	void LockSharedData(bool bLock);
	void FetchAndClearTagSizes(FMapActiveTagsToAllocationGroupTrackingData& OutDatas, int64* OutAllocTypeAmounts,
		EPruningLevel PruningLevel);

	void ClearAllocTypeAmounts();

	FActiveTags GetActiveTags(FLLMGlobals& LLMRef) const;
	void BeginActiveSetsChange(
		TArray<FAllocationGroupTrackingData, FDefaultLLMAllocator>& AllocationGroupTrackingDatas);
	void EndActiveSetsChange(const FActiveTags& NewDefaults,
		TArray<FAllocationGroupTrackingData, FDefaultLLMAllocator>& AllocationGroupTrackingDatas);

	// Either LLMRef.AllocationGroupsMutex or this->ThreadAllocationGroupsMutex must be held around read/write
	// of ActiveTagsToTrackingData and AllocTypeAmounts, because the Update thread enters a write lock on those
	// two locks (AllocationGroupsMutex entered first) when read/writing ActiveTagsToTrackingData and
	// AllocTypeAmounts from the update thread. ThreadAllocationGroupsMutex should be preferred when possible to avoid
	// blocking other threads. All other data on FLLMThreadState are accessed only from the thread that owns the
	// FLLMThreadState and do not require a lock.
	// These locks are also used to implement our contract that we guarantee for the Update thread's modification
	// of TagDatas and AllocationGroups which it does while holding FLLMTracker::LockAllThreadsSharedData: we must not
	// dereference any FTagData* or FAllocationGroup* held by the FLLMThreadState unless we have entered the global
	// AllocationGroupsMutex.
	FConstTagDataArray TagStack[static_cast<int32>(ELLMTagSet::Max)];
	FMapActiveTagsToAllocationGroupTrackingData ActiveTagsToTrackingData;
	UE::FPlatformRecursiveMutex ThreadAllocationGroupsMutex;

	int8 PausedCounter[(int32)ELLMAllocType::Count];
	int64 AllocTypeAmounts[(int32)ELLMAllocType::Count];
};

/** The main LLM implementation class. It owns the thread state objects. */
class FLLMTracker
{
public:
	struct FInPtr
	{
		FInPtr(const void* InPtr)
			: Ptr(InPtr) {
#if PLATFORM_STORES_DATA_IN_POINTER_HIGH_BITS
			// The values in the high 16 bits of Ptr are used to label the allocation, and are not needed to identify the pointer.
			// No two pointers will have the same values for the lower 48 bits but different values for the upper 16. We want to use
			// the upper 16 bits to store extra data of our own in our table that holds the pointer, so zero those bits out before using
			// them in our AllocationMap.
			static_assert(sizeof(void*) == 8);
			Ptr = reinterpret_cast<const void*>(reinterpret_cast<uintptr_t>(Ptr) & 0x0000'ffff'ffff'ffff);
#endif
		}
		operator const void* () 
		{ 
			return Ptr; 
		}
		const void* Ptr;
	};

	FLLMTracker(FLLMGlobals& InLLM);
	~FLLMTracker();

	void Initialize(ELLMTracker InTracker, FLLMAllocator* InAllocator);

	void PushTag(ELLMTag EnumTag, ELLMTagSet TagSet);
	void PushTag(FName Tag, bool bInIsStatTag, ELLMTagSet TagSet);
	void PushTag(const FTagData* TagData, ELLMTagSet TagSet);
	void PopTag(ELLMTagSet TagSet);
	void TrackAllocation(FInPtr Ptr, int64 Size, FName DefaultTag, ELLMAllocType AllocType, bool bTrackInMemPro);
	void TrackAllocation(FInPtr Ptr, int64 Size, ELLMTag DefaultTag, ELLMAllocType AllocType, bool bTrackInMemPro);
	void TrackFree(FInPtr Ptr, ELLMAllocType AllocType, bool bTrackInMemPro);
	void TrackMemoryOfActiveTag(int64 Amount, FName DefaultTag, ELLMAllocType AllocType);
	void TrackMemoryOfActiveTag(int64 Amount, ELLMTag DefaultTag, ELLMAllocType AllocType);
	void OnAllocMoved(FInPtr Dest, FInPtr Source, ELLMAllocType AllocType);

	void TrackMemory(ELLMTag EnumTag, int64 Amount, ELLMAllocType AllocType);
	void TrackMemory(FName TagName, ELLMTagSet TagSet, int64 Amount, ELLMAllocType AllocType);
	void TrackMemory(const FTagData* TagData, int64 Amount, ELLMAllocType AllocType);

	// This will pause/unpause tracking, and also manually increment a given tag
	void PauseAndTrackMemory(FName TagName, ELLMTagSet TagSet, bool bInIsStatTag, int64 Amount, ELLMAllocType AllocType);
	void PauseAndTrackMemory(ELLMTag EnumTag, int64 Amount, ELLMAllocType AllocType);
	void PauseAndTrackMemory(const FTagData* TagData, int64 Amount, ELLMAllocType AllocType);
	void Pause(ELLMAllocType AllocType);
	void Unpause(ELLMAllocType AllocType);
	bool IsPaused(ELLMAllocType AllocType);

	void Clear();

	// Dump the allocation count and size for each tag, along with the proportion that is private/shared/unreferenced
	// on linux machines. This will dump to a CSV in the project Saved/LLM directory with a separate file for each
	// tracker and forked child. Integers will be appended to the filename to prevent overwrites. Returns false
	// if the memory information can't be retrieved (likely because it's not on a supported platform)
	bool DumpForkedAllocationInfo();

	// Dump the list of all registered LLM tags and their properties to a CSV file, in the LLM's profiling directory
	// (Saved/Profiling/LLM/); see UE::LLM::GetLLMProfilingDir().
	bool DumpTags();

	void PublishStats(UE::LLM::ESizeParams SizeParam);
	void PublishCsv(UE::LLM::ESizeParams SizeParam);
	void PublishTrace(UE::LLM::ESizeParams SizeParam);
	void PublishCsvProfiler(UE::LLM::ESizeParams SizeParam);

	void OnPreFork();

	struct FLowLevelAllocInfo
	{
	public:
		void SetGroup(FAllocationGroup* AllocationGroup);
		FAllocationGroup* GetGroup(FLLMGlobals& InLLMRef) const;
		int32 GetCompressedTag() const;

	private:
#if LLM_ALLOW_NAMES_TAGS
		// Even with name-based tags we are still partially compressed - the allocation records the AllocationGroup's
		// index (4 bytes) rather than the full AllocationGroup pointer (8 bytes).
		int32 Group = -1;
#else
		ELLMTag Tag = ELLMTag::Untagged;
#endif
	};

	typedef LLMMap<PointerKey, uint32, FLowLevelAllocInfo, LLMNumAllocsType> FLLMAllocMap;  // pointer, size, info, Capacity SizeType

	void SetTotalTags(const FTagData* InOverrideUntaggedTagData, const FTagData* InOverrideTrackedTotalTagData);
	void Update();
	void UpdateThreads();
	void OnTagsResorted(FTagDataArray& OldTagDatas);
	void LockAllThreadsSharedData(bool bLock);
	void BeginActiveSetsChange(
		TArray<TArray<FAllocationGroupTrackingData, FDefaultLLMAllocator>, FDefaultLLMAllocator>& ThreadStateAllocationGroupTrackingDatas);
	void EndActiveSetsChange(const FActiveTags& NewDefaults,
		TArray<TArray<FAllocationGroupTrackingData, FDefaultLLMAllocator>, FDefaultLLMAllocator>& ThreadStateAllocationGroupTrackingDatas);

	void FetchAndClearTagSizes(bool bUpdatePruning);
	void CaptureTagSnapshot();
	void ClearTagSnapshot();

	int64 GetTagAmount(const FTagData* TagData, UE::LLM::ESizeParams SizeParams = UE::LLM::ESizeParams::Default) const;
	void SetTagAmountExternal(const FTagData* TagData, int64 Amount, bool bAddToTotal);
	void SetTagAmountInUpdate(const FTagData* TagData, int64 Amount, bool bAddToTotal);
	const FTagData* GetActiveTagData(ELLMTagSet TagSet = ELLMTagSet::None);
	TArray<const FTagData*> GetTagDatas(ELLMTagSet TagSet = ELLMTagSet::None);

	void GetTagsNamesWithAmount(TMap<FName, uint64>& OutTagsNamesWithAmount, ELLMTagSet TagSet = ELLMTagSet::None);
	void GetTagsNamesWithAmountFiltered(TMap<FName, uint64>& OutTagsNamesWithAmount, ELLMTagSet TagSet, TArray<FLLMTagSetAllocationFilter>& Filters);

	bool FindTagsForPtr(FInPtr InPtr, TArray<const FTagData*, TInlineAllocator<static_cast<int32>(ELLMTagSet::Max)>>& OutTags) const;

	int64 GetAllocTypeAmount(ELLMAllocType AllocType);

	int64 GetTrackedTotal(UE::LLM::ESizeParams SizeParams = UE::LLM::ESizeParams::Default) const
	{
		if (EnumHasAnyFlags(SizeParams, UE::LLM::ESizeParams::RelativeToSnapshot))
		{
			return FMath::Clamp<int64>(TrackedTotal - TrackedTotalInSnapshot, 0, INT64_MAX); //-V569
		}
		return TrackedTotal;
	}

	/**
	 * Slow function for callers that need to examine every allocation. Thread-safe. Call outside of any LLM locks, and
	 * in callback do not call LLM functions or allocate normally (use LLM internal allocator instead).
	 */
	void EnumerateAllocations(TFunctionRef<void(void* Ptr, int64 Size, FAllocationGroup* Group)> Callback);

protected:
	FLLMThreadState* GetOrCreateState();
	FLLMThreadState* GetState();
	void TrackAllocationInActiveTags(FInPtr Ptr, int64 Size, ELLMAllocType AllocType, FLLMThreadState* State, bool bTrackInMemPro);
	void TrackMemoryOfActiveTag(int64 Amount, const FTagData* TagData, ELLMAllocType AllocType, FLLMThreadState* State);
	void FetchAndClearTagSizes_UpdateGroupAllocs(FAllocationGroup& AllocationGroup,
		const FAllocationGroupTrackingData& TrackingData, EPruningLevel PruningLevel);
	FLLMGlobals& LLMRef;

	ELLMTracker Tracker;

	uint32 TlsSlot;

	TArray<FLLMThreadState*, FDefaultLLMAllocator> ThreadStates;

	FCriticalSection PendingThreadStatesGuard;
	TArray<FLLMThreadState*, FDefaultLLMAllocator> PendingThreadStates;
	/**
	 * Backup map from thread to threadstate. The primary lookup method is FPlatformTLS, but that is unavailable
	 * during thread termination on some platforms. When unavailable, we use this TMap (under PendingThreadStatesGuard
	 * critical section) to find the state.
	 */
	TMap<uint32, FLLMThreadState*, FDefaultSetLLMAllocator> ThreadIdToThreadState;

	/** Sum of memory from all tracked tags. Duplicated in separate storage to make it instantly available at any time of frame without waiting for accumulation from threads during update.
		GCC_ALIGN is required because it is modified in FPlatformAtomics::InterlockedAdd, which requires aligned values. */
	int64 TrackedTotal GCC_ALIGN(8);

	// The total tracked memory when the snapshot was taken
	int64 TrackedTotalInSnapshot;

#if !UE_ONLY_USE_PLATFORM_TRACKER
	FLLMAllocMap AllocationMap;
#endif

	FTrackerTagSizeMap TagSizes;

	const FTagData* OverrideUntaggedTagData;
	const FTagData* OverrideTrackedTotalTagData;

	FLLMCsvWriter CsvWriter;
	FLLMTraceWriter TraceWriter;
	FLLMCsvProfilerWriter CsvProfilerWriter;

	double LastTrimTime;

	int64 AllocTypeAmounts[(int32)ELLMAllocType::Count];
};

/**
 * The singleton implementation LLM class. It contains multiple FLLMTrackers, and contains storage for data shared
 * between trackers, such as FTagDatas.
 */
class FLLMGlobals : public FLowLevelMemTracker
{
public:
	// Functions Forwarded to from FLowLevelMemTracker
	void ProcessCommandLineInner(const TCHAR* CmdLine);
	uint64 GetTotalTrackedMemoryInner(ELLMTracker Tracker);
	void OnLowLevelAllocInner(ELLMTracker Tracker, const void* Ptr, uint64 Size, ELLMTag DefaultTag = ELLMTag::Untagged,
		ELLMAllocType AllocType = ELLMAllocType::None, bool bTrackInMemPro = true);
	void OnLowLevelAllocInner(ELLMTracker Tracker, const void* Ptr, uint64 Size, FName DefaultTag,
		ELLMAllocType AllocType = ELLMAllocType::None, bool bTrackInMemPro = true);
	void OnLowLevelFreeInner(ELLMTracker Tracker, const void* Ptr,
		ELLMAllocType AllocType = ELLMAllocType::None, bool bTrackInMemPro = true);
	void OnLowLevelChangeInMemoryUseInner(ELLMTracker Tracker, int64 DeltaMemory, ELLMTag DefaultTag = ELLMTag::Untagged, ELLMAllocType AllocType = ELLMAllocType::None);
	void OnLowLevelChangeInMemoryUseInner(ELLMTracker Tracker, int64 DeltaMemory, FName DefaultTag, ELLMAllocType AllocType = ELLMAllocType::None);
	void OnLowLevelAllocMovedInner(ELLMTracker Tracker, const void* Dest, const void* Source,
		ELLMAllocType AllocType = ELLMAllocType::None);
	void UpdateStatsPerFrameInner(const TCHAR* LogName = nullptr);
	void TickInner();
	void SetProgramSizeInner(uint64 InProgramSize);
	bool ExecInner(const TCHAR* Cmd, FOutputDevice& Ar);
	bool IsTagSetActiveInner(ELLMTagSet Set);
	bool IsTagSetScopeActiveInner(ELLMTagSet Set);
	bool IsTagSetRecordingActiveInner(ELLMTagSet Set);
	bool ShouldReduceThreadsInner();
	const FTagData* GetActiveTagDataInner(ELLMTracker Tracker, ELLMTagSet TagSet = ELLMTagSet::None);
	void RegisterPlatformTagInner(int32 Tag, const TCHAR* Name, FName StatName, FName SummaryStatName, int32 ParentTag = -1);
	void RegisterProjectTagInner(int32 Tag, const TCHAR* Name, FName StatName, FName SummaryStatName, int32 ParentTag = -1);
	TArray<const FTagData*> GetTrackedTagsInner(ELLMTagSet TagSet = ELLMTagSet::None);
	TArray<const FTagData*> GetTrackedTagsInner(ELLMTracker Tracker, ELLMTagSet TagSet = ELLMTagSet::None);
	void GetTrackedTagsNamesWithAmountInner(TMap<FName, uint64>& TagsNamesWithAmount, ELLMTracker Tracker, ELLMTagSet TagSet);
	void GetTrackedTagsNamesWithAmountFilteredInner(TMap<FName, uint64>& TagsNamesWithAmount, ELLMTracker Tracker, ELLMTagSet TagSet, TArray<FLLMTagSetAllocationFilter>& Filters);
	bool FindTagByNameInner(const TCHAR* Name, uint64& OutTag, ELLMTagSet InTagSet = ELLMTagSet::None) const;
	FName FindTagDisplayNameInner(uint64 Tag) const;
	FName FindPtrDisplayNameInner(void* Ptr) const;
	FName GetTagDisplayNameInner(const FTagData* TagData) const;
	FString GetTagDisplayPathNameInner(const FTagData* TagData) const;
	void GetTagDisplayPathNameInner(const FTagData* TagData,
		FStringBuilderBase& OutPathName, int32 MaxLen = -1) const;
	FName GetTagUniqueNameInner(const FTagData* TagData) const;
	const FTagData* GetTagParentInner(const FTagData* TagData) const;
	bool GetTagIsEnumTagInner(const FTagData* TagData) const;
	ELLMTag GetTagClosestEnumTagInner(const FTagData* TagData) const;
	int64 GetTagAmountForTrackerInner(ELLMTracker Tracker, ELLMTag Tag, UE::LLM::ESizeParams SizeParams = UE::LLM::ESizeParams::Default);
	int64 GetTagAmountForTrackerInner(ELLMTracker Tracker, const FTagData* TagData, UE::LLM::ESizeParams SizeParams = UE::LLM::ESizeParams::Default);
	int64 GetTagAmountForTrackerInner(ELLMTracker Tracker, FName Tag, ELLMTagSet TagSet, UE::LLM::ESizeParams SizeParams = UE::LLM::ESizeParams::Default);
	void SetTagAmountForTrackerInner(ELLMTracker Tracker, ELLMTag Tag, int64 Amount, bool bAddToTotal);
	void SetTagAmountForTrackerInner(ELLMTracker Tracker, FName Tag, ELLMTagSet TagSet, int64 Amount, bool bAddToTotal);
	uint64 DumpTagInner(ELLMTracker Tracker, const char* FileName, int LineNumber);
	void PublishDataSingleFrameInner();
	void DumpToLogInner(EDumpFormat DumpFormat = EDumpFormat::PlainText, FOutputDevice* OutputDevice = nullptr, UE::LLM::ESizeParams SizeParams = UE::LLM::ESizeParams::Default, ELLMTagSet TagSet = ELLMTagSet::None);
	void OnPreForkInner();
	bool IsInitializedInner() const;
	bool IsConfiguredInner() const;
	void BootstrapInitializeInner();
	void FinishInitializeInner();

	// Functions for use by types within LLMPrivate
	/** Accessor for the singleton FLLMGlobals for LLM code that does not already have a reference to it. */
	static FLLMGlobals& GetInner();
	/** Return the index in FActiveTags structures that corresponds to TagSet, or -1 if TagSet is not enabled. */
	int32 GetIndexInFActiveTags(ELLMTagSet TagSet);

private:
	FLLMGlobals();
	~FLLMGlobals();

	bool IsBootstrapping() const;

	/** Free all memory. This will put the tracker into a permanently disabled state. */
	void Clear();
	void InitializeProgramSize();

	class FLLMTracker* GetTracker(ELLMTracker Tracker);
	const class FLLMTracker* GetTracker(ELLMTracker Tracker) const;

	void TickInnerNoLock();
	void UpdateTags();
	void SortTags(FTagDataArray*& OutOldTagDatas);
	void PublishDataPerFrame(const TCHAR* LogName);

	void RegisterCustomTag(int32 Tag, ELLMTagSet TagSet, const TCHAR* Name, FName StatName, FName SummaryStatName, int32 ParentTag = -1);
	/**
	* Called during C++ global static initialization when GMalloc is available (and hence FNames are unavailable)
	* Creates the subset of tags necessary to record allocations during GMalloc and FName construction
	*/
	void BootstrapTagDatas();
	void BootstrapAllocationGroups();
	void InitializeTagDatas_SetLLMTagNames();
	void InitializeTagDatas_FinishRegister();
	void InitializeTagDatas();
	void ClearTagDatas();
	void ClearAllocationGroups();
	void OnActiveSetsInitialized(const FTagData* DefaultTags[static_cast<uint8>(ELLMTagSet::Max)]);
	void RegisterTagDeclaration(FLLMTagDeclaration& TagDeclaration);
	FTagData& RegisterTagData(FName Name, FName DisplayName, FName ParentName, FName StatName, FName SummaryStatName,
		bool bHasEnumTag, ELLMTag EnumTag, bool bIsStatTag, ETagReferenceSource ReferenceSource,
		ELLMTagSet TagSet = ELLMTagSet::None);
	/** Construct if not yet done the data on the given FTagData that relies on the presence of other TagDatas */
	void FinishConstruct(FTagData* TagData, ETagReferenceSource ReferenceSource);
	void ReportDuplicateTagName(FTagData* TagData, ETagReferenceSource ReferenceSource);

	const FTagData* FindOrAddTagData(ELLMTag EnumTag,
		ETagReferenceSource ReferenceSource = ETagReferenceSource::FunctionAPI);
	const FTagData* FindOrAddTagData(FName Name, ELLMTagSet TagSet, bool bIsStatData = false,
		ETagReferenceSource ReferenceSource = ETagReferenceSource::FunctionAPI);
	const FTagData* FindOrAddTagData(FName Name, ELLMTagSet TagSet, FName StatName,
		ETagReferenceSource ReferenceSource = ETagReferenceSource::FunctionAPI);
	const FTagData* FindTagData(ELLMTag EnumTag,
		ETagReferenceSource ReferenceSource = ETagReferenceSource::FunctionAPI);
	const FTagData* FindTagData(FName Name, ELLMTagSet TagSet,
		ETagReferenceSource ReferenceSource = ETagReferenceSource::FunctionAPI);
	const FTagData* FindOrAddDefaultTagData(ELLMTagSet TagSet);
	const FTagData* FindOrAddCodeOrContentCodeTagData();
	const FTagData* FindOrAddCodeOrContentContentTagData();
	FAllocationGroup* FindOrAddAllocationGroup(const UE::LLMPrivate::FActiveTags& ActiveTags);
	FAllocationGroup* FindAllocationGroupForCompressedAllocInfo(ELLMTag Tag);

	friend class ::FLLMPauseScope;
	friend class ::FLLMScope;
	friend class ::FLLMScopeDynamic;
	friend class ::FLLMScopeFromPtr;
	friend class ::FLowLevelMemTracker;
	friend class UE::LLMPrivate::FLLMCsvWriter;
	friend class UE::LLMPrivate::FLLMTracker;
	friend class UE::LLMPrivate::FLLMThreadState;
	friend class UE::LLMPrivate::FLLMTraceWriter;
	friend class UE::LLMPrivate::FLLMCsvProfilerWriter;
	friend void ::GlobalRegisterTagDeclaration(FLLMTagDeclaration& TagDeclaration);

private:
	FLLMAllocator Allocator;
	/** All TagDatas that have been constructed, in an array sorted by TagData->GetIndex() */
	FTagDataArray* TagDatas;
	/** Map from TagData->GetName() to TagData for all names, used to handle LLM_SCOPE with FName */
	FTagDataNameMap* TagDataNameMap;
	/** Array to Map from ELLMTag to the TagData for that tag, used to handle LLM_SCOPE with ELLMTag */
	FTagData** TagDataEnumMap;
	/** All AllocationGroups that have been constructed, indexed by AllocationGroup->GetIndex() */
	FAllocationGroupArray* AllocationGroups;
	/** AllocationGroups that were pruned and reset, used as an allocation pool. */
	FAllocationGroupArray* AllocationGroupsFreeList;
	/** AllocationGroups that have 0 refcount and should be considered for pruning. */
	FAllocationGroupSet* AllocationGroupsUnreferencedSet;
	/** Mapping from TagSetValue to AllocationGroup, only contains active AllocationGroups. */
	FMapActiveTagsToAllocationGroup* ActiveTagsToAllocationGroup;

	UE::LLMPrivate::FTagFilter* TraceTagFilter;

	FLLMTracker* Trackers[static_cast<int32>(ELLMTracker::Max)];

	mutable UE::FPlatformSharedMutex TagDataMutex;
	/**
	 * Guards ActiveTagsToAllocationGroup, AllocationGroupsFreeList, and AllocationGroupsUnreferencedSet. Potentially
	 * read-locked or write-locked on every allocation; threads cache the read of the containers to avoid contention.
	 **/
	mutable UE::FPlatformSharedMutex AllocationGroupsMutex;
	/** Guards AllocationGroups. Read-locked on every call to Free, so we avoid write-locking it whenever possible .*/
	mutable UE::FPlatformSharedMutex AllocationGroupsIndexMutex;
	UE::FPlatformRecursiveMutex UpdateMutex;
	UE::FPlatformRecursiveMutex BootstrapMutex;

	uint64 ProgramSize;
	int64 MemoryUsageCurrentOverhead;
	int64 MemoryUsagePlatformTotalUntracked;

	bool ActiveSets[(int32)ELLMTagSet::Max];

	bool bFirstTimeUpdating;
	bool bCanEnable;
	bool bCsvWriterEnabled;
	bool bTraceWriterEnabled;
	bool bInitializedTracking;
	bool bIsBootstrapping;
	bool bBeganInitializedTracking = false;
	bool bFullyInitialized;
	std::atomic<bool> bConfigurationComplete;
	bool bTagAdded;
	bool bAutoPublish;
	bool bPublishSingleFrame;
	bool bCapturedSizeSnapshot;
};

} // namespace UE::LLMPrivate


///////////////////////////////////////////////////////
// Inline implementations
///////////////////////////////////////////////////////


inline UE::LLMPrivate::FLLMGlobals* FLowLevelMemTracker::GetGlobals()
{
	return static_cast<UE::LLMPrivate::FLLMGlobals*>(this);
}

inline const UE::LLMPrivate::FLLMGlobals* FLowLevelMemTracker::GetGlobals() const
{
	return static_cast<const UE::LLMPrivate::FLLMGlobals*>(this);
}

namespace UE::LLMPrivate
{

inline bool FLLMGlobals::IsInitializedInner() const
{
	return bFullyInitialized;
}

inline bool FLLMGlobals::IsConfiguredInner() const
{
	return bConfigurationComplete;
}

inline bool FLLMGlobals::IsBootstrapping() const
{
	return bIsBootstrapping;
}

inline FLLMGlobals& FLLMGlobals::GetInner()
{
	return *FLowLevelMemTracker::Get().GetGlobals();
}

} // namespace UE::LLMPrivate

#endif		// #if ENABLE_LOW_LEVEL_MEM_TRACKER
