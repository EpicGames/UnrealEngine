// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/LowLevelMemTracker.h"

#if ENABLE_LOW_LEVEL_MEM_TRACKER

// Private defines configuring LLM; see also public defines in LowLevelMemTracker.h

// LLM_ALLOW_NAMES_TAGS: Set to 0 to reduce memory per allocation from 20 bytes to 17 bytes, at the cost of a reduced
// set of tags. Only ELLMTags will be displayed; all Name-based tags will be merged into their parent ELLMTag; most
// of them will be merged into the catch-all CustomName tag. LLMTagSets and tags created by the Stat system are not
// available when this setting is disabled.
#ifndef LLM_ALLOW_NAMES_TAGS
#define LLM_ALLOW_NAMES_TAGS 1
#endif

// Whether to enable running with reduced threads. This is currently enabled because the engine crashes with -norenderthread
#define LLM_ENABLED_REDUCE_THREADS 0

// this controls if the commandline is used to enable tracking, or to disable it. If LLM_COMMANDLINE_ENABLES_FUNCTIONALITY is true, 
// then tracking will only happen through Engine::Init(), at which point it will be disabled unless the commandline tells 
// it to keep going (with -llm). If LLM_COMMANDLINE_ENABLES_FUNCTIONALITY is false, then tracking will be on unless the commandline
// disables it (with -nollm)
#ifndef LLM_COMMANDLINE_ENABLES_FUNCTIONALITY
#define LLM_COMMANDLINE_ENABLES_FUNCTIONALITY 1
#endif 

// when set to 1, forces LLM to be enabled without having to parse the command line.
#ifndef LLM_AUTO_ENABLE
#define LLM_AUTO_ENABLE 0
#endif

// There is a little memory and cpu overhead in tracking peak memory but it is generally more useful than current memory.
// Disable if you need a little more memory or speed
#define LLM_ENABLED_TRACK_PEAK_MEMORY 1

// Enable to allow memory tracking to be passed through to external tools without the extra overhead of LLM also tracking allocations.
#define UE_ONLY_USE_PLATFORM_TRACKER 0

#define LLM_CSV_PROFILER_WRITER_ENABLED CSV_PROFILER_STATS

namespace UE::LLMPrivate
{

// PruneUpdatesForTrackingData must be >= 2 for behavior correctness, see comment in
// FLLMTracker::FetchAndClearTagSizes.
constexpr int32 PruneUpdatesForTrackingData = 5;
constexpr int32 PruneUpdatesForGroup = 5;
constexpr float PrunePopulationRatioForTrackingData = 2.5f;

enum class EPruningLevel : uint8
{
	None,
	Default,
	Max,
};

enum class ETagReferenceSource : uint8
{
	Scope,
	Declare,
	EnumTag,
	CustomEnumTag,
	FunctionAPI,
	ImplicitParent
};

} // namespace UE::LLMPrivate

#endif		// #if ENABLE_LOW_LEVEL_MEM_TRACKER
