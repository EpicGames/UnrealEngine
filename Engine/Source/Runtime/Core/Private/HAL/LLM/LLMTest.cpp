// Copyright Epic Games, Inc. All Rights Reserved.
#include "HAL/LowLevelMemTracker.h"

#if ENABLE_LOW_LEVEL_MEM_TRACKER

#include "HAL/LLM/LLMPrivate.h"

#include "Containers/Set.h"
#include "Misc/AutomationTest.h"

LLM_DEFINE_TAG(LLMTestParentTag_LLMTestChildTag1, NAME_None, TEXT("LLMTestParentTag")); // A test tag to verify that child tags can be declared before parent tags
LLM_DEFINE_TAG(LLMTestParentTag, NAME_None, NAME_None);
LLM_DEFINE_TAG(LLMTestParentTag_LLMTestChildTag2, NAME_None, TEXT("LLMTestParentTag")); // A test tag to verify that child tags can be declared after parent tags
LLM_DEFINE_TAG(LLMTestParentTag_LLMTestChildTag3); // A test tag to verify that child tags parse their parent tags

#if WITH_DEV_AUTOMATION_TESTS

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FLLMScopeTest, "System.Core.LLM.Scope", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter);

bool FLLMScopeTest::RunTest(const FString& Parameters)
{
	int* AllocationByNameExisting = nullptr;
	int* AllocationByNameNew = nullptr;
	int* AllocationByTag = nullptr;
	int* AllocationByTag3 = nullptr;
	FLowLevelMemTracker& LocalLLM = FLowLevelMemTracker::Get();
	bool bIsEnabled = FLowLevelMemTracker::IsEnabled();
	{
		LLM_SCOPE_BYNAME(TEXT("LLMTestParentTag/LLMTestChildTag1"));
		AllocationByNameExisting = new int[10];
		if (bIsEnabled)
		{
			TestEqual(TEXT("NameExisting has proper name"), LocalLLM.GetActiveTagData(ELLMTracker::Default)->GetName(), FName("LLMTestParentTag_LLMTestChildTag1"));
			TestEqual(TEXT("NameExisting has proper parent"), LocalLLM.GetActiveTagData(ELLMTracker::Default)->GetParent()->GetName(), FName("LLMTestParentTag"));
		}
	}
	{
		LLM_SCOPE_BYNAME(TEXT("LLMTestParentTag/LLMTestChildTag4"));
		AllocationByNameNew = new int[10];
		if (bIsEnabled)
		{
			TestEqual(TEXT("NameNew has proper name"), LocalLLM.GetActiveTagData(ELLMTracker::Default)->GetName(), FName("LLMTestParentTag_LLMTestChildTag4"));
			TestEqual(TEXT("NameNew has proper parent"), LocalLLM.GetActiveTagData(ELLMTracker::Default)->GetParent()->GetName(), FName("LLMTestParentTag"));
		}
	}
	{
		LLM_SCOPE_BYTAG(LLMTestParentTag_LLMTestChildTag2);
		AllocationByTag = new int[10];
		if (bIsEnabled)
		{
			TestEqual(TEXT("ByTag has proper name"), LocalLLM.GetActiveTagData(ELLMTracker::Default)->GetName(), FName("LLMTestParentTag_LLMTestChildTag2"));
			TestEqual(TEXT("ByTag has proper parent"), LocalLLM.GetActiveTagData(ELLMTracker::Default)->GetParent()->GetName(), FName("LLMTestParentTag"));
		}
	}
	{
		LLM_SCOPE_BYTAG(LLMTestParentTag_LLMTestChildTag3);
		AllocationByTag3 = new int[10];
		if (bIsEnabled)
		{
			TestEqual(TEXT("Parsed parent tag has proper name"), LocalLLM.GetActiveTagData(ELLMTracker::Default)->GetName(), FName("LLMTestParentTag_LLMTestChildTag3"));
			TestEqual(TEXT("Parsed parent tag has proper parent"), LocalLLM.GetActiveTagData(ELLMTracker::Default)->GetParent()->GetName(), FName("LLMTestParentTag"));
		}
	}
	delete[] AllocationByNameExisting;
	delete[] AllocationByNameNew;
	delete[] AllocationByTag;
	delete[] AllocationByTag3;

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FLLMMiscTest, "System.Core.LLM.Misc", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter);

bool FLLMMiscTest::RunTest(const FString& Parameters)
{
	// Test LLMGetTagUniqueNames
	TSet<FName> UniqueNames;
	for (int32 Tag = 0; Tag < static_cast<int32>(LLM_TAG_COUNT); ++Tag)
	{
		FName Name = LLMGetTagUniqueName(static_cast<ELLMTag>(Tag));
		if ((0 <= Tag && Tag < static_cast<int32>(ELLMTag::GenericTagCount)) || (LLM_CUSTOM_TAG_START <= Tag && Tag <= LLM_CUSTOM_TAG_END))
		{
			bool bAlreadySet;
			UniqueNames.Add(Name, &bAlreadySet);
			TestTrue(TEXT("LLMGetTagUniqueNames are unique"), !Name.IsNone() && !bAlreadySet);
		}
		else
		{
			TestTrue(TEXT("LLMGetTagUniqueName returns None for invalid tags"), Name.IsNone());
		}
	}
	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FTopologicalSortTest, "System.Core.LLM.TopologicalSort", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter);

bool FTopologicalSortTest::RunTest(const FString& Parameters)
{
	TArray<TArray<int32>> Edges;
	auto AppendEdges = [&Edges](int32 InVertex, int32* OutEdges, int32& OutNumEdges)
	{
		TArray<int32>& VertexEdges = Edges[InVertex];
		OutNumEdges = VertexEdges.Num();
		if (OutNumEdges)
		{
			FMemory::Memcpy(OutEdges, VertexEdges.GetData(), OutNumEdges*sizeof(VertexEdges[0]));
		}
	};

	// Simple case:
	//							0
	//						|		|
	//						v		v
	//						1  ->	2
	{
		TArray<TArray<int32>> RootToLeafEdges;
		RootToLeafEdges.Add(TArray<int32>{ 1, 2});
		RootToLeafEdges.Add(TArray<int32>{ 2 });
		RootToLeafEdges.AddDefaulted();

		Edges = RootToLeafEdges;
		LLMAlgo::TopologicalSortRootToLeaf(Edges, AppendEdges);
		TestTrue(TEXT("SimpleCase pre-sorted RootToLeaf remains stable sorted"), Edges == RootToLeafEdges);

		TArray<TArray<int32>> ReverseEdges;
		for (int n = RootToLeafEdges.Num() - 1; n >= 0; --n)
		{
			ReverseEdges.Add(RootToLeafEdges[n]);
		}
		LLMAlgo::TopologicalSortLeafToRoot(Edges, AppendEdges);
		TestTrue(TEXT("SimpleCase reverse-sorted RootToLeaf sorts into reverse order"), Edges == ReverseEdges);
		
		TArray<TArray<int32>> LeafToRootEdges;
		LeafToRootEdges.AddDefaulted();
		LeafToRootEdges.Add(TArray<int32>{ 0});
		LeafToRootEdges.Add(TArray<int32>{ 1,0 });
		Edges = LeafToRootEdges;
		LLMAlgo::TopologicalSortLeafToRoot(Edges, AppendEdges);
		TestTrue(TEXT("SimpleCase pre-sorted LeafToRoof remains stable sorted"), Edges == LeafToRootEdges);

		ReverseEdges.Reset();
		for (int n = LeafToRootEdges.Num() - 1; n >= 0; --n)
		{
			ReverseEdges.Add(LeafToRootEdges[n]);
		}
		LLMAlgo::TopologicalSortRootToLeaf(Edges, AppendEdges);
		TestTrue(TEXT("SimpleCase reverse-sorted LeafToRoof sorts into reverse order"), Edges == ReverseEdges);
	}

	// Simple cycle
	//							0
	//						|		|
	//						v		v
	//						1	->	2
	//							<-	|
	//								v
	//								3								
	{
		TArray<TArray<int32>> CycleEdges;
		CycleEdges.Add(TArray<int32>{ 1, 2});
		CycleEdges.Add(TArray<int32>{ 2 });
		CycleEdges.Add(TArray<int32>{ 1,3 });
		CycleEdges.AddDefaulted();

		Edges = CycleEdges;
		LLMAlgo::TopologicalSortLeafToRoot(Edges, AppendEdges);
		TestTrue(TEXT("Cycle correctly sorts target of cycle first in LeafToRoot order"), Edges[0] == CycleEdges[3]);
		TestTrue(TEXT("Cycle correctly sorts referencer of cycle last in LeafToRoot order"), Edges[3] == CycleEdges[0]);
		TestTrue(TEXT("Cycle correctly sorts vertex1 of cycle at an unspecified location into the middle"), Edges[1] == CycleEdges[1] || Edges[1] == CycleEdges[2]);
		TestTrue(TEXT("Cycle correctly sorts vertex2 of cycle at an unspecified location into the middle"), (Edges[2] == CycleEdges[1] || Edges[2] == CycleEdges[2]) && Edges[2] != Edges[1]);
	}

	return !HasAnyErrors();
}

#endif // WITH_DEV_AUTOMATION_TESTS

#endif // ENABLE_LOW_LEVEL_MEM_TRACKER