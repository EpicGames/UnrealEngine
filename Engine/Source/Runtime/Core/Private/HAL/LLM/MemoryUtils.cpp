// Copyright Epic Games, Inc. All Rights Reserved.

#include "HAL/LLM/MemoryUtils.h"

#include "CoreGlobals.h"

#if ENABLE_LOW_LEVEL_MEM_TRACKER

FORCENOINLINE void UE::Core::Private::OnInvalidLLMAllocatorNum(int32 IndexSize, int64 NewNum, SIZE_T NumBytesPerElement)
{
	UE_LOG(LogCore, Fatal, TEXT("Trying to resize TSizedLLMAllocator<%d> to an invalid size of % " INT64_FMT " with element size %" SIZE_T_FMT), IndexSize, NewNum, NumBytesPerElement);
	for (;;);
}

template void TSizedLLMAllocator<32>::ForAnyElementType::ResizeAllocation(SizeType CurrentNum, SizeType NewMax, SIZE_T NumBytesPerElement);
template void TSizedLLMAllocator<64>::ForAnyElementType::ResizeAllocation(SizeType CurrentNum, SizeType NewMax, SIZE_T NumBytesPerElement);

#endif
