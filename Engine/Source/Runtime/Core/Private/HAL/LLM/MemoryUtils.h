// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/LowLevelMemTracker.h"

#if ENABLE_LOW_LEVEL_MEM_TRACKER

#include "Algo/BinarySearch.h"
#include "Async/UniqueLock.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "HAL/LLM/LLMAllocator.h"
#include "HAL/PlatformMutex.h"
#include "Math/NumericLimits.h"
#include "Templates/AlignmentTemplates.h"
#include "Templates/Tuple.h"

#include <atomic>
#include <type_traits>

#define LLM_PAGE_SIZE (16*1024)

#if WITH_EDITOR
// When cooking, the number of simultaneous allocations can reach the danger zone of tens of millions, and our margin*capacity calculation ~ 100*capacity will rise over MAX_uint32
typedef uint64 LLMNumAllocsType;
#else
// Even in our 64 bit runtimes, the number of simultaneous allocations we have never gets over a few million, so we don't reach the danger zone of 100*capacity > MAX_uInt32
typedef uint32 LLMNumAllocsType;
#endif

// Trivially movable types without destructors only; memcpy is used and destructors are not called
template<typename T, typename SizeType=int32>
class FLLMArray
{
public:
	FLLMArray()
		: Array(StaticArray)
		, Count(0)
		, Capacity(StaticArrayCapacity)
		, Allocator(nullptr)
	{
	}

	~FLLMArray()
	{
		Clear(true);
	}

	void SetAllocator(UE::LLMPrivate::FLLMAllocator* InAllocator)
	{
		Allocator = InAllocator;
	}

	SizeType Num() const
	{
		return Count;
	}

	void Clear(bool ReleaseMemory = false)
	{
		if (ReleaseMemory)
		{
			if (Array != StaticArray)
			{
				Allocator->Free(Array, Capacity * sizeof(T));
				Array = StaticArray;
			}
			Capacity = StaticArrayCapacity;
		}
		Count = 0;
	}

	void Add(const T& Item)
	{
		if (Count == Capacity)
		{
			SizeType NewCapacity = DefaultCapacity;
			if (Capacity)
			{
				NewCapacity = Capacity + (Capacity / 2);
				ensureMsgf(NewCapacity > Capacity, TEXT("Unsigned integer overflow."));
			}
			Reserve(NewCapacity);
		}

		Array[Count] = Item;
		++Count;
	}

	T RemoveLast()
	{
		LLMCheck(Count > 0);
		--Count;
		T Last = Array[Count];

		return Last;
	}

	T& operator[](SizeType Index)
	{
		LLMCheck(Index >= 0 && Index < Count);
		return Array[Index];
	}

	const T& operator[](SizeType Index) const
	{
		LLMCheck(Index >= 0 && Index < Count);
		return Array[Index];
	}

	const T* GetData() const {
		return Array;
	}

	T& GetLast()
	{
		LLMCheck(Count > 0);
		return Array[Count - 1];
	}

	void Reserve(SizeType NewCapacity)
	{
		if (NewCapacity == Capacity)
		{
			return;
		}

		if (NewCapacity <= StaticArrayCapacity)
		{
			if (Array != StaticArray)
			{
				if (Count)
				{
					memcpy(StaticArray, Array, Count * sizeof(T));
				}
				Allocator->Free(Array, Capacity * sizeof(T));

				Array = StaticArray;
				Capacity = StaticArrayCapacity;
			}
		}
		else
		{
			NewCapacity = AlignArbitrary(NewCapacity, ItemsPerPage);

			T* NewArray = (T*)Allocator->Alloc(NewCapacity * sizeof(T));

			if (Count)
			{
				memcpy(NewArray, Array, Count * sizeof(T));
			}
			if (Array != StaticArray)
			{
				Allocator->Free(Array, Capacity * sizeof(T));
			}

			Array = NewArray;
			Capacity = NewCapacity;
		}
	}

	void operator=(const FLLMArray<T>& Other)
	{
		Clear();
		Reserve(Other.Count);
		memcpy(Array, Other.Array, Other.Count * sizeof(T));
		Count = Other.Count;
	}

	void Trim()
	{
		// Trim if usage has dropped below 3/4 of the total capacity
		if (Array != StaticArray && Count < (Capacity - (Capacity / 4)))
		{
			Reserve(Count);
		} 
	}

private:
	T* Array;
	SizeType Count;
	SizeType Capacity;

	UE::LLMPrivate::FLLMAllocator* Allocator;

	static const int StaticArrayCapacity = 64;	// because the default size is so large this actually saves memory
	T StaticArray[StaticArrayCapacity];

	static const int ItemsPerPage = LLM_PAGE_SIZE / sizeof(T);
	static const int DefaultCapacity = ItemsPerPage;
};

/*
* hash map
*/
template<typename TKey, typename TValue1, typename TValue2, typename SizeType=int32>
class LLMMap
{
public:
	typedef LLMMap<TKey, TValue1, TValue2, SizeType> ThisType;

	struct Values
	{
		TKey Key;
		TValue1 Value1;
		TValue2 Value2;
	};

	LLMMap()
		: bAllStripesLocked(false)
	{
	}

	~LLMMap()
	{
		Clear();
	}

	void SetAllocator(UE::LLMPrivate::FLLMAllocator* InAllocator, SizeType InDefaultCapacity = DefaultCapacity)
	{
		for (StripeData& Stripe : MapStripes)
		{
			Stripe.Allocator = InAllocator;

			Stripe.Keys.SetAllocator(InAllocator);
			Stripe.KeyHashes.SetAllocator(InAllocator);
			Stripe.Values1.SetAllocator(InAllocator);
			Stripe.Values2.SetAllocator(InAllocator);
			Stripe.FreeKeyIndices.SetAllocator(InAllocator);

			Stripe.Reserve(InDefaultCapacity / StripeCount);
		}
	}

	void Clear()
	{
		for (StripeData& Stripe : MapStripes)
		{
			UE::TUniqueLock AllocationScopeLock(Stripe.Mutex);

			Stripe.Keys.Clear(true);
			Stripe.KeyHashes.Clear(true);
			Stripe.Values1.Clear(true);
			Stripe.Values2.Clear(true);
			Stripe.FreeKeyIndices.Clear(true);

			Stripe.Allocator->Free(Stripe.Map, Stripe.Capacity * sizeof(SizeType));
			Stripe.Map = NULL;
			Stripe.Count = 0;
			Stripe.Capacity = 0;
		}
	}

	// Add a value to this set.
	// If this set already contains the value does nothing.
	void Add(const TKey& Key, const TValue1& Value1, const TValue2& Value2)
	{
		SizeType KeyHash = Key.GetHashCode();
		StripeData& Stripe = MapStripes[GetStripeIndex(KeyHash)];

		UE::TUniqueLock AllocationScopeLock(Stripe.Mutex);

		LLMCheck(Stripe.Map);

		SizeType MapIndex = Stripe.GetMapIndex(Key, KeyHash);
		SizeType KeyIndex = Stripe.Map[MapIndex];

		if (KeyIndex != InvalidIndex)
		{
			static bool ShownWarning = false;
			if (!ShownWarning)
			{
				FPlatformMisc::LowLevelOutputDebugString(TEXT("LLM WARNING: Replacing allocation in tracking map. Alloc/Free Mismatch.\n"));
				ShownWarning = true;
			}

			Stripe.Values1[KeyIndex] = Value1;
			Stripe.Values2[KeyIndex] = Value2;
		}
		else
		{
			// Be careful with the multiplication to avoid overflow, while still using only integer arithmetic for speed
			// Margin/256U is the actual float we want to multiply by. The exact number doesn't matter, so use one order of operations
			// when Capacity is low, and another when it is high
			SizeType MaxCount = (Stripe.Capacity >= 256U * 256U ? (Stripe.Capacity / 256U) * Margin : (Margin * Stripe.Capacity) / 256U);
			if (Stripe.Count >= MaxCount)
			{
				if (Stripe.Capacity * 2 < Stripe.Capacity)
				{
					// Trying to issue a check statement here will cause reentry into this function, use PLATFORM_BREAK directly instead
					FPlatformMisc::LowLevelOutputDebugString(TEXT("LLM Error: Integer overflow in LLMap::Add, Capacity has reached its maximum size.\n"));
					PLATFORM_BREAK();
				}
				if (Stripe.Count > MaxCount)
				{
					// Trying to issue a check statement here will cause reentry into this function, use PLATFORM_BREAK directly instead
					// This shouldn't happen, because Count is only incremented here, and Capacity is only changed here, and Margin does not change, so Count should equal MaxCount before it goes over it
					FPlatformMisc::LowLevelOutputDebugString(TEXT("LLM Assertion failure: Count > MaxCount.\n"));
					PLATFORM_BREAK();
				}
				Stripe.Grow();
				MapIndex = Stripe.GetMapIndex(Key, KeyHash);
			}

			if (Stripe.FreeKeyIndices.Num())
			{
				SizeType FreeIndex = Stripe.FreeKeyIndices.RemoveLast();
				Stripe.Map[MapIndex] = FreeIndex;
				Stripe.Keys[FreeIndex] = Key;
				Stripe.KeyHashes[FreeIndex] = KeyHash;
				Stripe.Values1[FreeIndex] = Value1;
				Stripe.Values2[FreeIndex] = Value2;
			}
			else
			{
				Stripe.Map[MapIndex] = Stripe.Keys.Num();
				Stripe.Keys.Add(Key);
				Stripe.KeyHashes.Add(KeyHash);
				Stripe.Values1.Add(Value1);
				Stripe.Values2.Add(Value2);
			}

			++Stripe.Count;
		}
	}

	Values GetValue(const TKey& Key)
	{
		Values RetValues;
		RetValues.Key = Find(Key, RetValues.Value1, RetValues.Value2);
		LLMEnsure(RetValues.Key);
		return RetValues;
	}

	TKey Find(const TKey& Key, TValue1& OutValue1, TValue2& OutValue2) const
	{
		SizeType KeyHash = Key.GetHashCode();
		const StripeData& Stripe = MapStripes[GetStripeIndex(KeyHash)];

		UE::TUniqueLock AllocationScopeLock(Stripe.Mutex);

		SizeType MapIndex = Stripe.GetMapIndex(Key, KeyHash);

		SizeType KeyIndex = Stripe.Map[MapIndex];
		if (KeyIndex == InvalidIndex)
		{
			return TKey();
		}

		OutValue1 = Stripe.Values1[KeyIndex];
		OutValue2 = Stripe.Values2[KeyIndex];

		return Stripe.Keys[KeyIndex];
	}

	bool Remove(const TKey& Key, Values& OutValues)
	{
		SizeType KeyHash = Key.GetHashCode();
		StripeData& Stripe = MapStripes[GetStripeIndex(KeyHash)];

		UE::TUniqueLock AllocationScopeLock(Stripe.Mutex);

		LLMCheck(Stripe.Map);

		SizeType MapIndex = Stripe.GetMapIndex(Key, KeyHash);
		if (!Stripe.IsItemInUse(MapIndex))
		{
			return false;
		}

		SizeType KeyIndex = Stripe.Map[MapIndex];

		OutValues.Key = Stripe.Keys[KeyIndex];
		OutValues.Value1 = Stripe.Values1[KeyIndex];
		OutValues.Value2 = Stripe.Values2[KeyIndex];

		if (KeyIndex == Stripe.Keys.Num() - 1)
		{
			Stripe.Keys.RemoveLast();
			Stripe.KeyHashes.RemoveLast();
			Stripe.Values1.RemoveLast();
			Stripe.Values2.RemoveLast();
		}
		else
		{
			Stripe.FreeKeyIndices.Add(KeyIndex);
		}

		// find first index in this array
		SizeType IndexIter = MapIndex;
		SizeType FirstIndex = MapIndex;
		if (!IndexIter)
		{
			IndexIter = Stripe.Capacity;
		}
		--IndexIter;
		while (Stripe.IsItemInUse(IndexIter))
		{
			FirstIndex = IndexIter;
			if (!IndexIter)
			{
				IndexIter = Stripe.Capacity;
			}
			--IndexIter;
		}

		bool Found = false;
		for (;;)
		{
			// find the last item in the array that can replace the item being removed
			SizeType IndexIter2 = (MapIndex + 1) & (Stripe.Capacity - 1);

			SizeType SwapIndex = InvalidIndex;
			while (Stripe.IsItemInUse(IndexIter2))
			{
				SizeType SearchKeyIndex = Stripe.Map[IndexIter2];
				const SizeType SearchHashCode = Stripe.KeyHashes[SearchKeyIndex];
				const SizeType SearchInsertIndex = SearchHashCode & (Stripe.Capacity - 1);

				if (Stripe.InRange(SearchInsertIndex, FirstIndex, MapIndex))
				{
					SwapIndex = IndexIter2;
					Found = true;
				}

				IndexIter2 = (IndexIter2 + 1) & (Stripe.Capacity - 1);
			}

			// swap the item
			if (Found)
			{
				Stripe.Map[MapIndex] = Stripe.Map[SwapIndex];
				MapIndex = SwapIndex;
				Found = false;
			}
			else
			{
				break;
			}
		}

		// remove the last item
		Stripe.Map[MapIndex] = InvalidIndex;

		--Stripe.Count;

		return true;
	}

	SizeType Num() const
	{
		SizeType TotalCount = 0;
		for (StripeData& Stripe : MapStripes)
		{
			UE::TUniqueLock AllocationScopeLock(Stripe.Mutex);
			TotalCount += Stripe.Count;
		}
		return TotalCount;
	}

	bool HasKey(const TKey& Key)
	{
		SizeType KeyHash = Key.GetHashCode();
		StripeData& Stripe = MapStripes[GetStripeIndex(KeyHash)];

		UE::TUniqueLock AllocationScopeLock(Stripe.Mutex);

		SizeType MapIndex = Stripe.GetMapIndex(Key, KeyHash);
		return Stripe.IsItemInUse(MapIndex);
	}

	void Trim()
	{
		for (StripeData& Stripe : MapStripes)
		{
			UE::TUniqueLock AllocationScopeLock(Stripe.Mutex);

			Stripe.Keys.Trim();
			Stripe.KeyHashes.Trim();
			Stripe.Values1.Trim();
			Stripe.Values2.Trim();
			Stripe.FreeKeyIndices.Trim();
		}
	}

	struct FBaseIterator
	{
	public:
		FBaseIterator& operator++()
		{
			++MapIndex;
			while (StripeIndex < StripeCount)
			{
				StripeData& Stripe = MapRef.MapStripes[StripeIndex];
				while (MapIndex < Stripe.Capacity)
				{
					if (Stripe.IsItemInUse(MapIndex))
					{
						return *this;
					}
					MapIndex++;
				}

				++StripeIndex;
				MapIndex = 0;
			}

			return *this;
		}


	protected:
		FBaseIterator(ThisType& InMap, bool bEnd)
			: MapRef(InMap), MapIndex(0)
		{
			if (bEnd)
			{
				StripeIndex = StripeCount;
				MapIndex = 0;
			}
			else
			{
				StripeIndex = 0;
				MapIndex = -1;
				++(*this);
			}
		}

		ThisType& MapRef;
		int32 StripeIndex;
		SizeType MapIndex;
	};

	struct FIterator;
	struct FTuple
	{
		const TKey& Key;
		TValue1& Value1;
		TValue2& Value2;

	private:
		FTuple(const TKey& InKey, TValue1& InValue1, TValue2& InValue2)
			:Key(InKey), Value1(InValue1), Value2(InValue2)
		{
		}

		friend struct ThisType::FIterator;
	};

	struct FIterator : public FBaseIterator
	{
		FIterator(ThisType& InMap, bool bEnd)
			: FBaseIterator(InMap, bEnd)
		{
		}

		bool operator!=(const FIterator& Other) const
		{
			return FBaseIterator::StripeIndex != Other.FBaseIterator::StripeIndex || 
				FBaseIterator::MapIndex != Other.FBaseIterator::MapIndex;
		}

		FTuple operator*() const
		{
			ThisType& LocalMap = FBaseIterator::MapRef;
			StripeData& Stripe = LocalMap.MapStripes[FBaseIterator::StripeIndex];
			SizeType KeyIndex = Stripe.Map[FBaseIterator::MapIndex];
			return FTuple(Stripe.Keys[KeyIndex], Stripe.Values1[KeyIndex], Stripe.Values2[KeyIndex]);
		}
	};

	struct FConstIterator;
	struct FConstTuple
	{
		const TKey& Key;
		const TValue1& Value1;
		const TValue2& Value2;

	private:
		FConstTuple(const TKey& InKey, const TValue1& InValue1, const TValue2& InValue2)
			:Key(InKey), Value1(InValue1), Value2(InValue2)
		{
		}

		friend struct ThisType::FConstIterator;
	};

	struct FConstIterator : public FBaseIterator
	{
		FConstIterator(const ThisType& InMap, bool bEnd)
			: FBaseIterator(const_cast<ThisType&>(InMap), bEnd)
		{
		}

		bool operator!=(const FConstIterator& Other) const
		{
			return FBaseIterator::StripeIndex != Other.FBaseIterator::StripeIndex || 
				FBaseIterator::MapIndex != Other.FBaseIterator::MapIndex;
		}

		FConstTuple operator*() const
		{
			ThisType& LocalMap = FBaseIterator::MapRef;
			StripeData& Stripe = LocalMap.MapStripes[FBaseIterator::StripeIndex];
			SizeType KeyIndex = Stripe.Map[FBaseIterator::MapIndex];
			return FConstTuple(Stripe.Keys[KeyIndex], Stripe.Values1[KeyIndex], Stripe.Values2[KeyIndex]);
		}
	};

	FIterator begin()
	{
		LLMCheck(bAllStripesLocked);
		return FIterator(*this, false);
	}

	FIterator end()
	{
		return FIterator(*this, true);
	}

	FConstIterator begin() const
	{
		LLMCheck(bAllStripesLocked);
		return FConstIterator(*this, false);
	}

	FConstIterator  end() const
	{
		return FConstIterator(*this, true);
	}

	void LockAll()
	{
		// Locking all stripes is required to create an iterator
		AllStripesMutex.Lock();
		LLMCheck(bAllStripesLocked == false);
		bAllStripesLocked = true;

		for (StripeData& Stripe : MapStripes)
		{
			Stripe.Mutex.Lock();
		}
	}

	void UnlockAll()
	{
		for (StripeData& Stripe : MapStripes)
		{
			Stripe.Mutex.Unlock();
		}

		LLMCheck(bAllStripesLocked == true);
		bAllStripesLocked = false;
		AllStripesMutex.Unlock();
	}

	// data
private:
	static const int32 StripeCountLog2 = 4;		// 16 stripes
	enum { StripeCount = 1 << StripeCountLog2 };
	enum { DefaultCapacity = 1024 * 1024 };
	enum { InvalidIndex = -1 };
	static const SizeType Margin = (30 * 256) / 100;

	static inline uint32 GetStripeIndex(uint32 KeyHash)
	{
		// FMath::Max prevents "shift too large" compile error in otherwise unreachable code when StripeCount == 1
		return (StripeCount == 1) ? 0 : KeyHash >> (32 - FMath::Max(StripeCountLog2, 1));
	}

	static inline uint32 GetStripeIndex(uint64 KeyHash)
	{
		// FMath::Max prevents "shift too large" compile error in otherwise unreachable code when StripeCount == 1
		return static_cast<uint32>((StripeCount == 1) ? 0 : KeyHash >> (64 - FMath::Max(StripeCountLog2, 1)));
	}

	struct StripeData
	{
		StripeData()
			: Allocator(nullptr)
			, Map(nullptr)
			, Count(0)
			, Capacity(0)
#ifdef PROFILE_LLMMAP
			, IterAcc(0)
			, IterCount(0)
#endif
		{}

		void Reserve(SizeType NewCapacity)
		{
			NewCapacity = FMath::Max(NewCapacity, (SizeType)1);
			NewCapacity = static_cast<SizeType>(FPlatformMath::RoundUpToPowerOfTwo64(static_cast<uint64>(NewCapacity)));

			// keep a copy of the old map
			SizeType* OldMap = Map;
			SizeType OldCapacity = Capacity;

			LLMCheck(NewCapacity > OldCapacity);

			// allocate the new table
			Capacity = NewCapacity;
			Map = (SizeType*)Allocator->Alloc(NewCapacity * sizeof(SizeType));

			for (SizeType Index = 0; Index < NewCapacity; ++Index)
				Map[Index] = InvalidIndex;

			// copy the values from the old to the new table
			SizeType* OldItem = OldMap;
			for (SizeType Index = 0; Index < OldCapacity; ++Index, ++OldItem)
			{
				SizeType KeyIndex = *OldItem;

				if (KeyIndex != InvalidIndex)
				{
					SizeType MapIndex = GetMapIndex(Keys[KeyIndex], KeyHashes[KeyIndex]);
					Map[MapIndex] = KeyIndex;
				}
			}

			Allocator->Free(OldMap, OldCapacity * sizeof(SizeType));
		}

		bool IsItemInUse(SizeType MapIndex) const
		{
			return Map[MapIndex] != InvalidIndex;
		}

		SizeType GetMapIndex(const TKey& Key, SizeType Hash) const
		{
			SizeType Mask = Capacity - 1;
			SizeType MapIndex = Hash & Mask;
			SizeType KeyIndex = Map[MapIndex];

			while (KeyIndex != InvalidIndex && !(Keys[KeyIndex] == Key))
			{
				MapIndex = (MapIndex + 1) & Mask;
				KeyIndex = Map[MapIndex];
#ifdef PROFILE_LLMMAP
				++IterAcc;
#endif
			}

#ifdef PROFILE_LLMMAP
			++IterCount;
			double Average = IterAcc / (double)IterCount;
			if (Average > 2.0)
			{
				static double LastWriteTime = 0.0;
				double Now = FPlatformTime::Seconds();
				if (Now - LastWriteTime > 5)
				{
					LastWriteTime = Now;
					UE_LOGF(LogStats, Log, "WARNING: LLMMap average: %f\n", (float)Average);
				}
			}
#endif
			return MapIndex;
		}

		// Increase the capacity of the map
		void Grow()
		{
			SizeType NewCapacity = Capacity ? 2 * Capacity : DefaultCapacity / StripeCount;
			Reserve(NewCapacity);
		}

		static bool InRange(
			const SizeType Index,
			const SizeType StartIndex,
			const SizeType EndIndex)
		{
			return (StartIndex <= EndIndex) ?
				Index >= StartIndex && Index <= EndIndex :
				Index >= StartIndex || Index <= EndIndex;
		}

		mutable UE::FPlatformRecursiveMutex Mutex;

		UE::LLMPrivate::FLLMAllocator* Allocator;

		SizeType* Map;
		SizeType Count;
		SizeType Capacity;

		// all these arrays must be kept in sync and are accessed by MapIndex
		FLLMArray<TKey, SizeType> Keys;
		FLLMArray<SizeType, SizeType> KeyHashes;
		FLLMArray<TValue1, SizeType> Values1;
		FLLMArray<TValue2, SizeType> Values2;

		FLLMArray<SizeType, SizeType> FreeKeyIndices;

#ifdef PROFILE_LLMMAP
		mutable int64 IterAcc;
		mutable int64 IterCount;
#endif
	};

	// We divide map items into multiple stripes by the high bits of the hash key, to reduce lock contention.
	// Each stripe has its own critical section lock, and given even distribution of hash codes (due to the
	// mix function applied to the key), multiple threads will usually end up with their own lock, and not
	// need to wait on each other.
	StripeData MapStripes[StripeCount];
	UE::FPlatformRecursiveMutex AllStripesMutex;
	std::atomic_bool bAllStripesLocked;
};

// Pointer key for hash map
struct PointerKey
{
	explicit PointerKey() = default;
	explicit PointerKey(const void* InPointer, uint16 ExtraData=0)
	: Pointer((UPTRINT(InPointer) << 16) | UPTRINT(ExtraData))
	{
		static_assert(sizeof(InPointer) == 8, "LLM only supports 64 bit platforms");
		LLMEnsure(UPTRINT(InPointer) && UPTRINT(InPointer) <= 0x0000'ffff'ffff'ffff);
	}
	LLMNumAllocsType GetHashCode() const
	{
		return GetHashCodeImpl<sizeof(LLMNumAllocsType), sizeof(void*)>();
	}
	bool operator==(const PointerKey& other) const { return GetPointer() == other.GetPointer(); }
	explicit operator bool () const { return !!Pointer; }
	void* GetPointer() const { return (void*)(Pointer >> 16); };
	uint64 GetExtraData() const { return uint64(Pointer & 0xffff); }

private:
	UPTRINT Pointer = 0;

	template <int HashSize, int PointerSize>
	LLMNumAllocsType GetHashCodeImpl() const
	{
		static_assert(HashSize == 0 && PointerSize == 0, "Converting void* to a LLMNumAllocsType - sized hashkey is not implemented for the current sizes.");
		return (LLMNumAllocsType)(UPTRINT)GetPointer();
	}

	template <>
	LLMNumAllocsType GetHashCodeImpl<8,8>() const
	{
		// 64 bit pointer to 64 bit hash
		uint64 Key = (uint64)GetPointer();
		Key = (~Key) + (Key << 21);
		Key = Key ^ (Key >> 24);
		Key = Key * 265;
		Key = Key ^ (Key >> 14);
		Key = Key * 21;
		Key = Key ^ (Key >> 28);
		Key = Key + (Key << 31);
		return (LLMNumAllocsType)Key;
	}

	template <>
	LLMNumAllocsType GetHashCodeImpl<4, 8>() const
	{
		// 64 bit pointer to 32 bit hash
		uint64 Key = (uint64)GetPointer();
		Key = (~Key) + (Key << 21);
		Key = Key ^ (Key >> 24);
		Key = Key * 265;
		Key = Key ^ (Key >> 14);
		Key = Key * 21;
		Key = Key ^ (Key >> 28);
		Key = Key + (Key << 31);
		return (LLMNumAllocsType)Key;
	}

	template <>
	LLMNumAllocsType GetHashCodeImpl<4, 4>() const
	{
		// 32 bit pointer to 32 bit Hash
		uint64 Key = (uint64)GetPointer();
		Key = (~Key) + (Key << 18);
		Key = Key ^ (Key >> 31);
		Key = Key * 21;
		Key = Key ^ (Key >> 11);
		Key = Key + (Key << 6);
		Key = Key ^ (Key >> 22);
		return (LLMNumAllocsType)Key;
	}
};

namespace UE::Core::Private
{
	[[noreturn]] CORE_API void OnInvalidLLMAllocatorNum(int32 IndexSize, int64 NewNum, SIZE_T NumBytesPerElement);
}

/**
 * An allocator usable in Core containers. It is based on TSizedHeapAllocator, but instead of allocating from FMemory it allocates from FLLMAllocator.
 * Because FLLMAllocator::Free requires the Size, this allocator also has a Size field that TSizedHeapAllocator does not.
 */
template<int IndexSize>
class TSizedLLMAllocator
{
public:
	using SizeType = typename TBitsToSizeType<IndexSize>::Type;

private:
	using USizeType = std::make_unsigned_t<SizeType>;

public:
	enum { NeedsElementType = false };
	enum { RequireRangeCheck = true };

	class ForAnyElementType
	{
		template <int>
		friend class TSizedLLMAllocator;

	public:
		/** Default constructor. */
		ForAnyElementType()
			: Data(nullptr)
			, Size(0)
		{}

		/**
		 * Moves the state of another allocator into this one.  The allocator can be different.
		 *
		 * Assumes that the allocator is currently empty, i.e. memory may be allocated but any existing elements have already been destructed (if necessary).
		 * @param Other - The allocator to move the state from.  This allocator should be left in a valid empty state.
		 */
		template <typename OtherAllocator>
		FORCEINLINE void MoveToEmptyFromOtherAllocator(typename OtherAllocator::ForAnyElementType& Other)
		{
			checkSlow((void*)this != (void*)&Other);

			if (Data)
			{
				UE::LLMPrivate::FLLMAllocator::Get()->Free(Data, Size);
			}

			Data = Other.Data;
			Size = Other.Size;
			Other.Data = nullptr;
			Other.Size = 0;
		}

		/**
		 * Moves the state of another allocator into this one.
		 * Moves the state of another allocator into this one.  The allocator can be different.
		 *
		 * Assumes that the allocator is currently empty, i.e. memory may be allocated but any existing elements have already been destructed (if necessary).
		 * @param Other - The allocator to move the state from.  This allocator should be left in a valid empty state.
		 */
		FORCEINLINE void MoveToEmpty(ForAnyElementType& Other)
		{
			this->MoveToEmptyFromOtherAllocator<TSizedLLMAllocator>(Other);
		}

		/** Destructor. */
		FORCEINLINE ~ForAnyElementType()
		{
			if (Data)
			{
				UE::LLMPrivate::FLLMAllocator::Get()->Free(Data, Size);
			}
		}

		FORCEINLINE void* GetAllocation() const
		{
			return Data;
		}

		FORCEINLINE void ResizeAllocation(SizeType CurrentNum, SizeType NewMax, SIZE_T NumBytesPerElement)
		{
			// Avoid calling FMemory::Realloc( nullptr, 0 ) as ANSI C mandates returning a valid pointer which is not what we want.
			if (Data || NewMax)
			{
				static_assert(sizeof(SizeType) <= sizeof(SIZE_T), "SIZE_T is expected to handle all possible sizes");

				// Check for under/overflow
				bool bInvalidResize = NewMax < 0 || NumBytesPerElement < 1 || NumBytesPerElement > (SIZE_T)MAX_int32;
				if constexpr (sizeof(SizeType) == sizeof(SIZE_T))
				{
					bInvalidResize = bInvalidResize || (SIZE_T)(USizeType)NewMax > (SIZE_T)TNumericLimits<SizeType>::Max() / NumBytesPerElement;
				}
				if (UNLIKELY(bInvalidResize))
				{
					UE::Core::Private::OnInvalidLLMAllocatorNum(IndexSize, NewMax, NumBytesPerElement);
				}

				//checkSlow(((uint64)NewMax*(uint64)ElementTypeInfo.GetSize() < (uint64)INT_MAX));
				size_t NewSize = NewMax * NumBytesPerElement;
				Data = UE::LLMPrivate::FLLMAllocator::Get()->Realloc(Data, Size, NewSize);
				Size = NewSize;
			}
		}
		FORCEINLINE SizeType CalculateSlackReserve(SizeType NewMax, SIZE_T NumBytesPerElement) const
		{
			return DefaultCalculateSlackReserve(NewMax, NumBytesPerElement, true);
		}
		FORCEINLINE SizeType CalculateSlackShrink(SizeType NewMax, SizeType CurrentMax, SIZE_T NumBytesPerElement) const
		{
			return DefaultCalculateSlackShrink(NewMax, CurrentMax, NumBytesPerElement, true);
		}
		FORCEINLINE SizeType CalculateSlackGrow(SizeType NewMax, SizeType CurrentMax, SIZE_T NumBytesPerElement) const
		{
			return DefaultCalculateSlackGrow(NewMax, CurrentMax, NumBytesPerElement, true);
		}

		SIZE_T GetAllocatedSize(SizeType CurrentMax, SIZE_T NumBytesPerElement) const
		{
			return CurrentMax * NumBytesPerElement;
		}

		bool HasAllocation() const
		{
			return !!Data;
		}

		SizeType GetInitialCapacity() const
		{
			return 0;
		}

	private:
		ForAnyElementType(const ForAnyElementType&);
		ForAnyElementType& operator=(const ForAnyElementType&);

		/** A pointer to the container's elements. */
		void* Data;
		/** The allocation size of Data */
		size_t Size;
	};

	template<typename ElementType>
	class ForElementType : public ForAnyElementType
	{
	public:

		/** Default constructor. */
		ForElementType()
		{}

		FORCEINLINE ElementType* GetAllocation() const
		{
			return (ElementType*)ForAnyElementType::GetAllocation();
		}
	};
};

// Define the ResizeAllocation functions with the regular allocators as exported to avoid bloat
extern template CORE_API FORCENOINLINE void TSizedLLMAllocator<32>::ForAnyElementType::ResizeAllocation(SizeType CurrentNum, SizeType NewMax, SIZE_T NumBytesPerElement);
extern template CORE_API FORCENOINLINE void TSizedLLMAllocator<64>::ForAnyElementType::ResizeAllocation(SizeType CurrentNum, SizeType NewMax, SIZE_T NumBytesPerElement);

// The standard container-specific allocators based on TSizedLLMAllocator; these are copied from ContainerAllocationPolicies.h
using FDefaultLLMAllocator = TSizedLLMAllocator<32>;
using FDefaultLLMAllocator64 = TSizedLLMAllocator<64>;

class FDefaultBitArrayLLMAllocator : public TInlineAllocator<4, FDefaultLLMAllocator> { public: typedef TInlineAllocator<4, FDefaultLLMAllocator>     Typedef; };
class FDefaultSparseArrayLLMAllocator : public TSparseArrayAllocator<FDefaultLLMAllocator, FDefaultBitArrayLLMAllocator> { public: typedef TSparseArrayAllocator<FDefaultLLMAllocator, FDefaultBitArrayLLMAllocator> Typedef; };
class FDefaultSetLLMAllocator : public TSetAllocator<FDefaultSparseArrayLLMAllocator, TInlineAllocator<1, FDefaultLLMAllocator>> { public: typedef TSetAllocator<FDefaultSparseArrayLLMAllocator, TInlineAllocator<1, FDefaultLLMAllocator>>         Typedef; };

template<typename KeyType, typename ValueType>
class TLLMMap : public TMap<KeyType, ValueType, FDefaultSetLLMAllocator>
{};

template<typename KeyType>
class TLLMSet : public TSet<KeyType, DefaultKeyFuncs<KeyType>, FDefaultSetLLMAllocator>
{};



// Some algorithms used by LLM that require scratch-space internal allocation. To avoid polluting
// the Algo namespace with an Allocator parameter, we've copied (or privately implemented the nonexisting ones) those
// algorithms here
namespace LLMAlgoImpl
{
	enum ETopologicalSortOrder
	{
		RootToLeaf,
		LeafToRoot
	};

	template <typename T, typename GetEdgesType, typename SizeType>
	void TopologicalSort(T* VertexData, SizeType NumVertices, GetEdgesType GetEdges, ETopologicalSortOrder SortOrder);
}

namespace LLMAlgo
{
	/**
	 * Sort a range of vertices topologically from root to leaf by the directed edges given by GetEdges.
	 * Vertices in cycles will be sorted in an arbitrary order relative to each other.
	 * The sort is stable.
	 *
	 * @param Range			The range of vertices to sort.
	 * @param GetEdges		A callable GetEdges(SizeType InVertex, SizeType* OutEdgeBuffer, Sizetype& OutNumEdges) that provides the indexes into VertexData of the edges for InVertex.
	 *						Edges should be written into OutEdgeBuffer; this buffer only has room for GetNum(Range) elements.
	 *						InVertex is the index in Range of the vertex.
	 *						OutEdgeBuffer should be populated with the indexes in Range of the target vertices.
	 */
	template <typename RangeType, typename GetEdgesType>
	void TopologicalSortRootToLeaf(RangeType&& Range, GetEdgesType GetEdges)
	{
		TopologicalSort(GetData(Range), GetNum(Range), GetEdges, LLMAlgoImpl::ETopologicalSortOrder::RootToLeaf);
	}

	/**
	 * Sort a range of vertices topologically from leaf to root by the directed edges given by GetEdges.
	 * Vertices in cycles will be sorted in an arbitrary order relative to each other.
	 * The sort is stable.
	 *
	 * @param Range			The range of vertices to sort.
	 * @param GetEdges		A callable GetEdges(SizeType InVertex, SizeType* OutEdgeBuffer, Sizetype& OutNumEdges) that provides the indexes into VertexData of the edges for InVertex.
	 *						Edges should be written into OutEdgeBuffer; this buffer only has room for GetNum(Range) elements.
	 *						InVertex is the index in Range of the vertex.
	 *						OutEdgeBuffer should be populated with the indexes in Range of the target vertices.
	 */
	template <typename RangeType, typename GetEdgesType>
	void TopologicalSortLeafToRoot(RangeType&& Range, GetEdgesType GetEdges)
	{
		using namespace LLMAlgoImpl;
		TopologicalSort(GetData(Range), GetNum(Range), GetEdges, ETopologicalSortOrder::LeafToRoot);
	}
}

namespace LLMAlgoImpl
{

	/**
	 * Sort an array of vertices topologically by the directed edges given by GetEdges.
	 * Vertices in cycles will be sorted in an arbitrary order relative to each other.
	 * The sort is stable.
	 *
	 * @param VertexData	Pointer to the vertices to sort.
	 * @param NumVertices	Number of vertices.
	 * @param GetEdges		A callable GetEdges(SizeType InVertex, SizeType* OutEdgeBuffer, Sizetype& OutNumEdges) that provides the indexes into VertexData of the edges for InVertex.
	 *						Edges should be written into OutEdgeBuffer; this buffer only has room for NumVertices elements.
	 *						InVertex is the index in Range of the vertex.
	 *						OutEdgeBuffer should be populated with the indexes in Range of the target vertices.
	 * @param SortOrder		Whether to order the in/out VertexData by sources before targets (RootToLeaf) or targets before sources (LeafToRoot).
	 */
	template <typename T, typename GetEdgesType, typename SizeType>
	void TopologicalSort(T* VertexData, SizeType NumVertices, GetEdgesType GetEdges, ETopologicalSortOrder SortOrder)
	{
		if (NumVertices == 0)
		{
			return;
		}
		LLMCheckf(NumVertices > 0, TEXT("Invalid number of vertices"));

		// In our traversal, we write vertices LeafToRoot. To make a stable sort, we need to iterate the input list from LeafToRoot as well.
		SizeType TraversalStart, TraversalEnd, TraversalDir;
		if (SortOrder == ETopologicalSortOrder::RootToLeaf)
		{
			TraversalStart = NumVertices - 1;
			TraversalEnd = -1;
			TraversalDir = -1;
		}
		else
		{
			TraversalStart = 0;
			TraversalEnd = NumVertices;
			TraversalDir = 1;
		}

		TArray<SizeType, FDefaultLLMAllocator> LeafToRootOrder;
		LeafToRootOrder.Reserve(NumVertices);

		{
			constexpr SizeType ExpectedMaxNumEdges = 16;
			struct FVisitData
			{
				SizeType Vertex;
				SizeType NextEdge;
				SizeType EdgeStart;
			};

			TArray<FVisitData, FDefaultLLMAllocator> Stack;
			Stack.Reserve(NumVertices);

			TArray<SizeType, FDefaultLLMAllocator> EdgeBuffer;
			EdgeBuffer.AddUninitialized(NumVertices);
			SizeType* EdgeBufferData = EdgeBuffer.GetData();

			TArray<SizeType, FDefaultLLMAllocator> EdgesOnStack;
			EdgesOnStack.Reserve(NumVertices * ExpectedMaxNumEdges);

			TArray<bool, FDefaultLLMAllocator> Visited;
			Visited.AddDefaulted(NumVertices);

			auto PushVertex = [&EdgesOnStack, &Stack, &GetEdges, &Visited, EdgeBufferData, NumVertices](SizeType Vertex)
			{
				Visited[Vertex] = true;
				Stack.Add(FVisitData{ Vertex, EdgesOnStack.Num(), EdgesOnStack.Num() });
				SizeType NumEdges = static_cast<SizeType>(-1);
				GetEdges(Vertex, EdgeBufferData, NumEdges);
				LLMCheckf(0 <= NumEdges && NumEdges <= NumVertices, TEXT("GetEdges function passed into TopologicalSort did not write OutNumEdges, or wrote an invalid value"));
				EdgesOnStack.Append(EdgeBufferData, NumEdges);
			};

			for (SizeType RootVertex = TraversalStart; RootVertex != TraversalEnd; RootVertex += TraversalDir)
			{
				if (Visited[RootVertex])
				{
					continue;
				}

				PushVertex(RootVertex);
				while (Stack.Num() > 0)
				{
					FVisitData& VisitData = Stack.Last();
					bool bPushed = false;
					SizeType NumEdgesOnStack = EdgesOnStack.Num();
					for (; VisitData.NextEdge < NumEdgesOnStack; ++VisitData.NextEdge)
					{
						SizeType TargetVertex = EdgesOnStack[VisitData.NextEdge];
						if (!Visited[TargetVertex])
						{
							++VisitData.NextEdge;
							PushVertex(TargetVertex);
							bPushed = true;
							break;
						}
					}
					if (!bPushed)
					{
						LeafToRootOrder.Add(VisitData.Vertex);
						EdgesOnStack.SetNum(VisitData.EdgeStart, EAllowShrinking::No);
						Stack.Pop(EAllowShrinking::No);
					}
				}
			}
		}
		LLMCheck(LeafToRootOrder.Num() == NumVertices); // This could only fail due to an internal logic error; all vertices should have been visited and added

		TArray<T, FDefaultLLMAllocator> Original;
		Original.Reserve(NumVertices);
		for (SizeType n = 0; n < NumVertices; ++n)
		{
			Original.Add(MoveTemp(VertexData[n]));
		}

		for (SizeType LeafToRootIndex = 0; LeafToRootIndex < NumVertices; ++LeafToRootIndex)
		{
			SizeType WriteIndex = TraversalStart + TraversalDir * LeafToRootIndex;
			SizeType ReadIndex = LeafToRootOrder[LeafToRootIndex];
			VertexData[WriteIndex] = MoveTemp(Original[ReadIndex]);
		}
	}
}

#endif		// #if ENABLE_LOW_LEVEL_MEM_TRACKER

