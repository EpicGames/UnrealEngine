// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MallocAnsi.cpp: Binned memory allocator
=============================================================================*/

#include "HAL/MallocAnsi.h"

#include "AutoRTFM.h"
#include "HAL/LowLevelMemTracker.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/AssertionMacros.h"
#include "ProfilingDebugging/MemoryTrace.h"
#include "Templates/AlignmentTemplates.h"

#if PLATFORM_USE_ANSI_POSIX_MALLOC
	#include <malloc.h>
#endif

#if PLATFORM_IOS
	#include "mach/mach.h"
#endif

#if PLATFORM_WINDOWS
	#include "Windows/WindowsHWrapper.h"
#endif

#define MALLOC_ANSI_USES__ALIGNED_MALLOC !UE_AUTORTFM && PLATFORM_USES__ALIGNED_MALLOC

static void* AnsiMallocInternal(SIZE_T Size, uint32 Alignment);
static void AnsiFreeInternal(void* Ptr);
static void* AnsiReallocInternal(void* Ptr, SIZE_T NewSize, uint32 Alignment);

static SIZE_T AnsiGetAllocationSize(void* Original)
{
#if MALLOC_ANSI_USES__ALIGNED_MALLOC
	return _aligned_msize(Original, 16, 0); // TODO: incorrectly assumes alignment of 16
#elif PLATFORM_USE_ANSI_POSIX_MALLOC || PLATFORM_USE_ANSI_MEMALIGN
	return malloc_usable_size(Original);
#else
	return *((SIZE_T*)((uint8*)Original - sizeof(void*) - sizeof(SIZE_T)));
#endif
}

void* AnsiMalloc(SIZE_T Size, uint32 Alignment)
{
	LLM_PLATFORM_SCOPE(ELLMTag::FMalloc);

	void* Result = AnsiMallocInternal(Size, Alignment);

	LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelAlloc(ELLMTracker::Default, Result, Size,
		ELLMTag::Untagged, ELLMAllocType::FMalloc));
	LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelAlloc(ELLMTracker::Platform, Result, Size));
	MemoryTrace_Alloc(uint64(Result), Size, Alignment, EMemoryTraceRootHeap::SystemMemory);

	return Result;
}

void* AnsiRealloc(void* Ptr, SIZE_T NewSize, uint32 Alignment)
{
	LLM_PLATFORM_SCOPE(ELLMTag::FMalloc);
	LLM_REALLOC_SCOPE(Ptr);
	LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelFree(ELLMTracker::Default, Ptr, ELLMAllocType::FMalloc));
	LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelFree(ELLMTracker::Platform, Ptr));
	MemoryTrace_ReallocFree(uint64(Ptr), EMemoryTraceRootHeap::SystemMemory);

	void* Result = AnsiReallocInternal(Ptr, NewSize, Alignment);

	MemoryTrace_ReallocAlloc(uint64(Result), NewSize, Alignment, EMemoryTraceRootHeap::SystemMemory);
	LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelAlloc(ELLMTracker::Platform, Result, NewSize));
	LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelAlloc(ELLMTracker::Default, Result, NewSize,
		ELLMTag::Untagged, ELLMAllocType::FMalloc));

	return Result;
}

void AnsiFree(void* Ptr)
{
	LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelFree(ELLMTracker::Default, Ptr, ELLMAllocType::FMalloc));
	LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelFree(ELLMTracker::Platform, Ptr));
	MemoryTrace_Free(uint64(Ptr), EMemoryTraceRootHeap::SystemMemory);

	AnsiFreeInternal(Ptr);
}

void* AnsiMallocUntracked(SIZE_T Size, uint32 Alignment)
{
	void* Result = AnsiMallocInternal(Size, Alignment);

	// MemoryTrace tracking has been implemented and tested with all known use cases, so we can call its tracking.
	// Do not add other tracking to AnsiMallocUntracked API without testing the known problem systems:
	//    RenderDoc, LLM
	MemoryTrace_Alloc(uint64(Result), Size, Alignment, EMemoryTraceRootHeap::SystemMemory);

	return Result;
}

void* AnsiReallocUntracked(void* Ptr, SIZE_T NewSize, uint32 Alignment)
{
	// MemoryTrace tracking has been implemented and tested with all known use cases, so we can call its tracking.
	// Do not add other tracking to AnsiMallocUntracked API without testing the known problem systems:
	//    RenderDoc, LLM
	MemoryTrace_ReallocFree(uint64(Ptr), EMemoryTraceRootHeap::SystemMemory);

	void* Result = AnsiReallocInternal(Ptr, NewSize, Alignment);

	MemoryTrace_ReallocAlloc(uint64(Result), NewSize, Alignment, EMemoryTraceRootHeap::SystemMemory);

	return Result;
}

void AnsiFreeUntracked(void* Ptr)
{
	// MemoryTrace tracking has been implemented and tested with all known use cases, so we can call its tracking.
	// Do not add other tracking to AnsiMallocUntracked API without testing the known problem systems:
	//    RenderDoc, LLM
	MemoryTrace_Free(uint64(Ptr), EMemoryTraceRootHeap::SystemMemory);

	AnsiFreeInternal(Ptr);
}

static void* AnsiMallocInternal(SIZE_T Size, uint32 Alignment)
{
#if MALLOC_ANSI_USES__ALIGNED_MALLOC
	void* Result = _aligned_malloc(Size, Alignment);
#elif PLATFORM_USE_ANSI_POSIX_MALLOC
	void* Result;
	if (UNLIKELY(posix_memalign(&Result, Alignment, Size) != 0))
	{
		Result = nullptr;
	}
#elif PLATFORM_USE_ANSI_MEMALIGN
	void* Result = memalign(Alignment, Size);
#else
	void* Ptr = malloc(Size + Alignment + sizeof(void*) + sizeof(SIZE_T));
	void* Result = nullptr;
	if (Ptr)
	{
		Result = Align((uint8*)Ptr + sizeof(void*) + sizeof(SIZE_T), Alignment);
		*((void**)((uint8*)Result - sizeof(void*))) = Ptr;
		*((SIZE_T*)((uint8*)Result - sizeof(void*) - sizeof(SIZE_T))) = Size;
	}
#endif

	return Result;
}

static void* AnsiReallocInternal(void* Ptr, SIZE_T NewSize, uint32 Alignment)
{
	void* Result;

#if MALLOC_ANSI_USES__ALIGNED_MALLOC
	if (Ptr && NewSize)
	{
		Result = _aligned_realloc(Ptr, NewSize, Alignment);
	}
	else if (Ptr == nullptr)
	{
		Result = _aligned_malloc(NewSize, Alignment);
	}
	else
	{
		_aligned_free(Ptr);
		Result = nullptr;
	}
#elif PLATFORM_USE_ANSI_POSIX_MALLOC
	if (Ptr && NewSize)
	{
		SIZE_T UsableSize = malloc_usable_size(Ptr);
		if (UNLIKELY(posix_memalign(&Result, Alignment, NewSize) != 0))
		{
			Result = nullptr;
		}
		else if (LIKELY(UsableSize))
		{
			FMemory::Memcpy(Result, Ptr, FMath::Min(NewSize, UsableSize));
		}
		free(Ptr);
	}
	else if (Ptr == nullptr)
	{
		if (UNLIKELY(posix_memalign(&Result, Alignment, NewSize) != 0))
		{
			Result = nullptr;
		}
	}
	else
	{
		free(Ptr);
		Result = nullptr;
	}
#elif PLATFORM_USE_ANSI_MEMALIGN
	Result = reallocalign(Ptr, NewSize, Alignment);
#else
	if (Ptr && NewSize)
	{
		// Can't use realloc as it might screw with alignment.
		Result = AnsiMallocInternal(NewSize, Alignment);
		SIZE_T PtrSize = AnsiGetAllocationSize(Ptr);
		FMemory::Memcpy(Result, Ptr, FMath::Min(NewSize, PtrSize));
		AnsiFreeInternal(Ptr);
	}
	else if (Ptr == nullptr)
	{
		Result = AnsiMallocInternal(NewSize, Alignment);
	}
	else
	{
		Result = nullptr;
		AnsiFreeInternal(Ptr);
	}
#endif

	return Result;
}

void AnsiFreeInternal(void* Ptr)
{
#if MALLOC_ANSI_USES__ALIGNED_MALLOC
	_aligned_free(Ptr);
#elif PLATFORM_USE_ANSI_POSIX_MALLOC || PLATFORM_USE_ANSI_MEMALIGN
	free(Ptr);
#else
	if (Ptr)
	{
		free(*((void**)((uint8*)Ptr - sizeof(void*))));
	}
#endif
}

FMallocAnsi::FMallocAnsi()
{
#if PLATFORM_WINDOWS
	// Enable low fragmentation heap - http://msdn2.microsoft.com/en-US/library/aa366750.aspx
	intptr_t	CrtHeapHandle = _get_heap_handle();
	ULONG		EnableLFH = 2;
	HeapSetInformation((void*)CrtHeapHandle, HeapCompatibilityInformation, &EnableLFH, sizeof(EnableLFH));
#endif
}

void* FMallocAnsi::TryMalloc(SIZE_T Size, uint32 Alignment)
{
#if !UE_BUILD_SHIPPING
	uint64 LocalMaxSingleAlloc = MaxSingleAlloc.Load(EMemoryOrder::Relaxed);
	if (LocalMaxSingleAlloc != 0 && Size > LocalMaxSingleAlloc)
	{
		return nullptr;
	}
#endif

	Alignment = FMath::Max(Size >= 16 ? (uint32)16 : (uint32)8, Alignment);

	void* Result;
	{
		LLM_PLATFORM_SCOPE(ELLMTag::FMalloc);

		Result = AnsiMallocInternal(Size, Alignment);

		// We need to report mallocs to LLM's platform tracker, but not to LLM's default tracker; FMemory.inl (or any other
		// caller of FMallocAnsi) is responsible for reporting to the default tracker.
		LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelAlloc(ELLMTracker::Platform, Result, Size));
		MemoryTrace_Alloc(uint64(Result), Size, Alignment, EMemoryTraceRootHeap::SystemMemory);
	}

	return Result;
}

void* FMallocAnsi::Malloc(SIZE_T Size, uint32 Alignment)
{
	void* Result = TryMalloc(Size, Alignment); 

	if (Result == nullptr && Size)
	{
		FPlatformMemory::OnOutOfMemory(Size, Alignment);
	}

	return Result;
}

void* FMallocAnsi::TryRealloc(void* Ptr, SIZE_T NewSize, uint32 Alignment)
{
#if !UE_BUILD_SHIPPING
	uint64 LocalMaxSingleAlloc = MaxSingleAlloc.Load(EMemoryOrder::Relaxed);
	if (LocalMaxSingleAlloc != 0 && NewSize > LocalMaxSingleAlloc)
	{
		return nullptr;
	}
#endif

	Alignment = FMath::Max(NewSize >= 16 ? (uint32)16 : (uint32)8, Alignment);

	void* Result;
	{
		// We need to report reallocs to LLM's platform tracker, but not to LLM's default tracker; FMemory.inl (or any other
		// caller of FMallocAnsi) is responsible for reporting to the default tracker.
		LLM_PLATFORM_SCOPE(ELLMTag::FMalloc);
		LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelFree(ELLMTracker::Platform, Ptr));
		MemoryTrace_ReallocFree(uint64(Ptr), EMemoryTraceRootHeap::SystemMemory);

		Result = AnsiReallocInternal(Ptr, NewSize, Alignment);

		MemoryTrace_ReallocAlloc(uint64(Result), NewSize, Alignment, EMemoryTraceRootHeap::SystemMemory);
		LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelAlloc(ELLMTracker::Platform, Result, NewSize));
	}

	return Result;
}

void* FMallocAnsi::Realloc( void* Ptr, SIZE_T NewSize, uint32 Alignment )
{
	void* Result = TryRealloc(Ptr, NewSize, Alignment);

	if (Result == nullptr && NewSize != 0)
	{
		FPlatformMemory::OnOutOfMemory(NewSize, Alignment);
	}

	return Result;
}

void FMallocAnsi::Free( void* Ptr )
{
	// We need to report frees to LLM's platform tracker, but not to LLM's default tracker; FMemory.inl (or any other
	// caller of FMallocAnsi) is responsible for reporting to the default tracker.
	LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelFree(ELLMTracker::Platform, Ptr));
	MemoryTrace_Free(uint64(Ptr), EMemoryTraceRootHeap::SystemMemory);

	AnsiFreeInternal(Ptr);
}

bool FMallocAnsi::GetAllocationSize(void* Original, SIZE_T& SizeOut)
{
	if (!Original)
	{
		return false;
	}

#if MALLOC_ANSI_USES__ALIGNED_MALLOC
	return false;
#else
	SizeOut = AnsiGetAllocationSize(Original);
	return true;
#endif
}

bool FMallocAnsi::IsInternallyThreadSafe() const
{
#if PLATFORM_IS_ANSI_MALLOC_THREADSAFE
		return true;
#else
		return false;
#endif
}

bool FMallocAnsi::ValidateHeap()
{
#if PLATFORM_WINDOWS
	int32 Result = _heapchk();
	check(Result != _HEAPBADBEGIN);
	check(Result != _HEAPBADNODE);
	check(Result != _HEAPBADPTR);
	check(Result != _HEAPEMPTY);
	check(Result == _HEAPOK);
#endif
	return true;
}
