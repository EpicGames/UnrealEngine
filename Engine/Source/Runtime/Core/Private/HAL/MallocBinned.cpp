// Copyright Epic Games, Inc. All Rights Reserved.

#include "HAL/MallocBinned.h"
#if PLATFORM_MAC
#include "Mac/MacSystemIncludes.h"
#elif PLATFORM_IOS
#include "IOS/IOSSystemIncludes.h"
#endif

#include "Async/UniqueLock.h"
#include "Misc/BufferedOutputDevice.h"

#include "HAL/MemoryMisc.h"

PRAGMA_DISABLE_UNSAFE_TYPECAST_WARNINGS

/** Malloc binned allocator specific stats. */
DEFINE_STAT(STAT_Binned_OsCurrent);
DEFINE_STAT(STAT_Binned_OsPeak);
DEFINE_STAT(STAT_Binned_WasteCurrent);
DEFINE_STAT(STAT_Binned_WastePeak);
DEFINE_STAT(STAT_Binned_UsedCurrent);
DEFINE_STAT(STAT_Binned_UsedPeak);
DEFINE_STAT(STAT_Binned_CurrentAllocs);
DEFINE_STAT(STAT_Binned_TotalAllocs);
DEFINE_STAT(STAT_Binned_SlackCurrent);

#if USE_OS_SMALL_BLOCK_GRAB_MEMORY_FROM_OS && ENABLE_LOW_LEVEL_MEM_TRACKER
DEFINE_STAT(STAT_Binned_NanoMallocPages_Current);
DEFINE_STAT(STAT_Binned_NanoMallocPages_Peak);
DEFINE_STAT(STAT_Binned_NanoMallocPages_WasteCurrent);
DEFINE_STAT(STAT_Binned_NanoMallocPages_WastePeak);
#endif

#if PLATFORM_IOS
#define PLAT_PAGE_SIZE_LIMIT 16384
#define PLAT_BINNED_ALLOC_POOLSIZE 16384
#define PLAT_SMALL_BLOCK_POOL_SIZE 256
#else
#define PLAT_PAGE_SIZE_LIMIT 65536
#define PLAT_BINNED_ALLOC_POOLSIZE 65536
#define PLAT_SMALL_BLOCK_POOL_SIZE 0
#endif //PLATFORM_IOS

/** Information about a piece of free memory. 16 bytes */
struct alignas(16) FMallocBinned::FFreeMem
{
	/** Next or MemLastPool[], always in order by pool. */
	FFreeMem*	Next;
	/** Number of consecutive free blocks here, at least 1. */
	uint32		NumFreeBlocks;
};

// Memory pool info. 32 bytes.
struct FMallocBinned::FPoolInfo
{
	/** Number of allocated elements in this pool, when counts down to zero can free the entire pool. */
	uint16			Taken;		// 2
	/** Index of pool. Index into MemSizeToPoolTable[]. Valid when < MAX_POOLED_ALLOCATION_SIZE, MAX_POOLED_ALLOCATION_SIZE is OsTable.
		When AllocSize is 0, this is the number of pages to step back to find the base address of an allocation. See FindPoolInfoInternal()
	*/
	uint16			TableIndex; // 4		
	/** Number of bytes allocated */
	uint32			AllocSize;	// 8
	/** Pointer to first free memory in this pool or the OS Allocation Size in bytes if this allocation is not binned*/
	FFreeMem*		FirstMem;   // 12/16
	FPoolInfo*		Next;		// 16/24
	FPoolInfo**		PrevLink;	// 20/32

	void SetAllocationSizes( uint32 InBytes, UPTRINT InOsBytes, uint32 InTableIndex, uint32 SmallAllocLimt )
	{
		TableIndex=InTableIndex;
		AllocSize=InBytes;
		if (TableIndex == SmallAllocLimt)
		{
			FirstMem=(FFreeMem*)InOsBytes;
		}
	}

	uint32 GetBytes() const
	{
		return AllocSize;
	}

	UPTRINT GetOsBytes( uint32 InPageSize, uint32 SmallAllocLimt ) const
	{
		if (TableIndex == SmallAllocLimt)
		{
			return (UPTRINT)FirstMem;
		}
		else
		{
			return Align(AllocSize, InPageSize);
		}
	}

	void Link( FPoolInfo*& Before )
	{
		if( Before )
		{
			Before->PrevLink = &Next;
		}
		Next     = Before;
		PrevLink = &Before;
		Before   = this;
	}

	void Unlink()
	{
		if( Next )
		{
			Next->PrevLink = PrevLink;
		}
		*PrevLink = Next;
	}
};

/** Hash table struct for retrieving allocation book keeping information */
struct FMallocBinned::PoolHashBucket
{
	UPTRINT			Key;
	FPoolInfo*		FirstPool;
	PoolHashBucket* Prev;
	PoolHashBucket* Next;

	PoolHashBucket()
	{
		Key=0;
		FirstPool=nullptr;
		Prev=this;
		Next=this;
	}

	void Link( PoolHashBucket* After )
	{
		Link(After, Prev, this);
	}

	static void Link( PoolHashBucket* Node, PoolHashBucket* Before, PoolHashBucket* After )
	{
		Node->Prev=Before;
		Node->Next=After;
		Before->Next=Node;
		After->Prev=Node;
	}

	void Unlink()
	{
		Next->Prev = Prev;
		Prev->Next = Next;
		Prev=this;
		Next=this;
	}
};

struct FMallocBinned::Private
{
	/** Default alignment for binned allocator */
	enum { DEFAULT_BINNED_ALLOCATOR_ALIGNMENT = sizeof(FFreeMem) };
	static_assert(DEFAULT_BINNED_ALLOCATOR_ALIGNMENT == 16, "Default alignment should be 16 bytes");
	enum { PAGE_SIZE_LIMIT = PLAT_PAGE_SIZE_LIMIT };
	// BINNED_ALLOC_POOL_SIZE can be increased beyond 64k to cause binned malloc to allocate
	// the small size bins in bigger chunks. If OS Allocation is slow, increasing
	// this number *may* help performance but YMMV.
	enum { BINNED_ALLOC_POOL_SIZE = PLAT_BINNED_ALLOC_POOLSIZE };
	// On IOS can push small allocs in to a pre-allocated small block pool
	enum { SMALL_BLOCK_POOL_SIZE = PLAT_SMALL_BLOCK_POOL_SIZE };

	static bool bHasInitializedStatsMetadata;
	
#if USE_OS_SMALL_BLOCK_ALLOC
#if USE_OS_SMALL_BLOCK_GRAB_MEMORY_FROM_OS
	enum { SMALL_BLOCK_MAX_TOTAL_POOL_SIZE = 0x20000000 };
	enum { SMALL_BLOCK_GRAB_ALLOC_ALIGN = DEFAULT_BINNED_ALLOCATOR_ALIGNMENT };
	enum { SMALL_BLOCK_GRAB_MAX_ALLOC_SIZE = 256 };
	enum { SMALL_BLOCK_GRAB_MIN_ALLOC_SIZE = SMALL_BLOCK_GRAB_ALLOC_ALIGN };
	
	enum { SMALL_BLOCK_GRAB_TEMP_MEM_ARRAY_SIZE = (SMALL_BLOCK_MAX_TOTAL_POOL_SIZE / SMALL_BLOCK_GRAB_MIN_ALLOC_SIZE) };
	
	//Sizes that we attempt to grab sequential allocated memory in
	enum { BLOCK_GRAB_TARGET_BIN_SIZE = PAGE_SIZE_LIMIT };
	
	//if set to true, we will only grab sequential blocks that are page aligned. If set, make sure our
	//BLOCK_GRAB_TARGET_BIN_SIZE above is set to something that is also a multiple of page size
	enum { SMALL_BLOCK_GRAB_ENSURE_PAGE_ALIGNMENT = true };
	
	//How much of the small block pool we should use max for our GRAB_MEMORY_FROM_OS override
	enum { MAX_SMALL_BLOCK_USEAGE_SIZE = 250 * 1024 * 1024 }; //250MB
	enum { MAX_NUM_BLOCK_START_ADDRESSES = MAX_SMALL_BLOCK_USEAGE_SIZE / BLOCK_GRAB_TARGET_BIN_SIZE };
	
	//Storage for all our free SmallBlockGrab free start addresses
	static uint64 SmallBlockGrab_FreeStartPointers[MAX_NUM_BLOCK_START_ADDRESSES];
	
	//How many actual allocations we have available in our SmallBlockGrab_FreeStartPointers array
	static int NumFreeSmallBlockGrabAllocations;
	
	static bool IsSmallBlockGrabAllocation(void* ptr)
	{
		return (   (((uint64)ptr) >= Private::SmallBlockStartPtr)
				&& (((uint64)ptr) < Private::SmallBlockEndPtr)  );
	}
	
	static bool IsNanoMallocPageAligned(void* ptr)
	{
		uint64 AddressOffset = ((uint64)ptr) - SmallBlockStartPtr;
		return ((AddressOffset % PAGE_SIZE_LIMIT) == 0);
	}

	inline static UE::FPlatformRecursiveMutex SmallLock;

	static FFreeMem* GetAllocFromSmallBlockGrab(FMallocBinned& Allocator, UPTRINT OsBytes)
	{
		//Make sure we can hold this memory in a free SmallBlockGrab
		if (OsBytes > Private::BLOCK_GRAB_TARGET_BIN_SIZE)
		{
			//FPlatformMisc::LowLevelOutputDebugStringf(TEXT("OSBytes: %d"), OsBytes);
			return nullptr;
		}
		
		UE::TUniqueLock ScopedLock(SmallLock);
		if (Private::NumFreeSmallBlockGrabAllocations == 0)
		{
			//FPlatformMisc::LowLevelOutputDebugStringf(TEXT("OSBytes: %d  FreeAllocations: %d"), OsBytes, Private::NumFreeSmallBlockGrabAllocations);
			return nullptr;
		}
		
		int GrabIndex = --NumFreeSmallBlockGrabAllocations;
		check(GrabIndex >= 0);
		void* FreeMemory = (void*)SmallBlockGrab_FreeStartPointers[GrabIndex];
		
		//Flag that we have used this value for verification
		SmallBlockGrab_FreeStartPointers[GrabIndex] = 0xBADF00D;
		
		BINNED_PEAK_STATCOUNTER(Allocator.NanoMallocPagesPeak,    BINNED_ADD_STATCOUNTER(Allocator.NanoMallocPagesCurrent, BLOCK_GRAB_TARGET_BIN_SIZE));
		BINNED_PEAK_STATCOUNTER(Allocator.NanoMallocWastePagesPeak, BINNED_ADD_STATCOUNTER(Allocator.NanoMallocPagesWaste, BLOCK_GRAB_TARGET_BIN_SIZE - OsBytes));
		
		return (FFreeMem*)FreeMemory;
	}
	
	static void FreeSmallBlockGrab(void* ptr, FMallocBinned& Allocator, SIZE_T Size)
	{
		UE::TUniqueLock ScopedLock(SmallLock);

		check(IsSmallBlockGrabAllocation(ptr));
		check(!SMALL_BLOCK_GRAB_ENSURE_PAGE_ALIGNMENT || IsNanoMallocPageAligned(ptr));
		
		int NewIndex = NumFreeSmallBlockGrabAllocations;
		++NumFreeSmallBlockGrabAllocations;
		check(NewIndex < MAX_NUM_BLOCK_START_ADDRESSES);
		
		SmallBlockGrab_FreeStartPointers[NewIndex] = (uint64) ptr;
			
		BINNED_ADD_STATCOUNTER(Allocator.NanoMallocPagesCurrent,    -(int64)BLOCK_GRAB_TARGET_BIN_SIZE);
		BINNED_ADD_STATCOUNTER(Allocator.NanoMallocPagesWaste, -(int64)(BLOCK_GRAB_TARGET_BIN_SIZE - Size));
	}
	
	//Helper struct for storing all our allocation data in an intermediate
	struct FMemoryAllocationGrabberHelper
	{
	public:
		uint32 NumFoundCombinedBlocks;
		
		FMemoryAllocationGrabberHelper()
		: NumFoundCombinedBlocks(0)
		{
			ResetArrays();
		}
		
		void AddGrabbedMemory(uint64 AddressAddedAt, uint32 AllocatedSize)
		{
			assert((AddressAddedAt > 0) && (AllocatedSize > 0) && (SmallBlockStartPtr > 0));
			{
				uint32 IndexToAddAt = 0;
				IndexToAddAt = (AddressAddedAt - SmallBlockStartPtr) / SMALL_BLOCK_GRAB_MIN_ALLOC_SIZE;
				
				assert((IndexToAddAt > 0) && (IndexToAddAt < SMALL_BLOCK_GRAB_TEMP_MEM_ARRAY_SIZE));
				{
					assert(GrabbedMemoryBlocks[IndexToAddAt] == 0);
					{
						GrabbedMemoryBlocks[IndexToAddAt] = AllocatedSize;
					}
				}
			}
		}
		
		void ValidateMemoryBlocks()
		{
			for (int Index = 0; Index < SMALL_BLOCK_GRAB_TEMP_MEM_ARRAY_SIZE;)
			{
				if (GrabbedMemoryBlocks[Index] > 0)
				{
					uint32 IndicesToParse = (GrabbedMemoryBlocks[Index] / SMALL_BLOCK_GRAB_MIN_ALLOC_SIZE);
					
					//Verify no data shows up before we would expect it in the array
					//Since the value is our data size, we shouldn't have allocated anything between our Start Index and our expected End Index
					for (int SizeVerifyIndex = (Index + 1); SizeVerifyIndex < IndicesToParse; ++SizeVerifyIndex)
					{
						assert(GrabbedMemoryBlocks[Index + SizeVerifyIndex] == 0);
					}
					
					//Go ahead and write a temp value in there just to test accessing all these memory blocks
					const uint64 MemoryAddress = GetAddressForIndex(Index);
					new ((void*)MemoryAddress) int(5);
					
					Index += IndicesToParse;
				}
				else
				{
					++Index;
				}
			}
		}
		
		uint64 GetAddressForIndex(uint32 Index)
		{
			return (Index * SMALL_BLOCK_GRAB_MIN_ALLOC_SIZE) + SmallBlockStartPtr;
		}
		
		void FindGrabbedBlockSequentials(uint64 OutArrayCombinedAddressStartPtrs[MAX_NUM_BLOCK_START_ADDRESSES])
		{
			//If we care about page alignment, we want to make sure the target sizes are a multiple of page alignment
			static_assert((!Private::SMALL_BLOCK_GRAB_ENSURE_PAGE_ALIGNMENT || ((Private::BLOCK_GRAB_TARGET_BIN_SIZE % PAGE_SIZE_LIMIT) == 0)),
						  "If SMALL_BLOCK_GRAB_ENSURE_PAGE_ALIGNMENT is true, then BLOCK_GRAB_TARGET_BIN_SIZE must be a multiple of PAGE_SIZE_LIMIT.");
			
			static uint64 CurrentSequentialBlockFoundCount = 0;
			static uint64 CurrentBlockStartIndex = 0;
			static bool CurrentBlockMeetsPageAlignmentRequirements = IsGrabbedBlockIndexPageAligned(0);
			
			for (int Index = 0; Index < SMALL_BLOCK_GRAB_TEMP_MEM_ARRAY_SIZE;)
			{
				bool bDidEndSequentialChain = false;
				if (CurrentBlockMeetsPageAlignmentRequirements && (GrabbedMemoryBlocks[Index] > 0))
				{
					CurrentSequentialBlockFoundCount += GrabbedMemoryBlocks[Index];
					
					//End this chain if we already have a large enough bucket
					bDidEndSequentialChain = (CurrentSequentialBlockFoundCount >= BLOCK_GRAB_TARGET_BIN_SIZE);
					
					uint32 IndicesToAdvance = (GrabbedMemoryBlocks[Index] / SMALL_BLOCK_GRAB_MIN_ALLOC_SIZE);
					Index += IndicesToAdvance;
				}
				else
				{
					bDidEndSequentialChain = true;
					++Index;
				}
				
				if (bDidEndSequentialChain)
				{
					//If we are large, save as a large sequential block as long as we haven't saved too many
					if ((CurrentSequentialBlockFoundCount >= BLOCK_GRAB_TARGET_BIN_SIZE) && (NumFoundCombinedBlocks < MAX_NUM_BLOCK_START_ADDRESSES))
					{
						OutArrayCombinedAddressStartPtrs[NumFoundCombinedBlocks] = GetAddressForIndex(CurrentBlockStartIndex);
						++NumFoundCombinedBlocks;
					}
					
					//if we didn't make it to either, go ahead and de-allocate memory in this chain
					else
					{
						for (int FreeIndex = CurrentBlockStartIndex; FreeIndex < Index;++FreeIndex)
						{
							//Only free memory that we actually allocated
							if (GrabbedMemoryBlocks[FreeIndex] > 0)
							{
								uint64 PointerToFree = GetAddressForIndex(FreeIndex);
								::free((void*)PointerToFree);
							}
						}
					}
					
					CurrentSequentialBlockFoundCount = 0;
					CurrentBlockStartIndex = Index;
					CurrentBlockMeetsPageAlignmentRequirements = IsGrabbedBlockIndexPageAligned(CurrentBlockStartIndex);
				}
			}
		}
		
		//Look if we are at the start of a page with this memory block index
		bool IsGrabbedBlockIndexPageAligned(int Index)
		{
			if (!Private::SMALL_BLOCK_GRAB_ENSURE_PAGE_ALIGNMENT)
			{
				return true;
			}
			
			const uint64 MemLoc = GetAddressForIndex(Index);
			const uint64 MemoryOffsetFromStart = MemLoc - SmallBlockStartPtr;
			
			return ((MemoryOffsetFromStart % PAGE_SIZE_LIMIT) == 0);
		}
		
	private:
		//Tracker holding the state of all possible memory allocations.
		//0 if un-allocated. Otherwise holds the data size allocated
		uint32 GrabbedMemoryBlocks[SMALL_BLOCK_GRAB_TEMP_MEM_ARRAY_SIZE];
		
		void ResetArrays()
		{
			memset(&GrabbedMemoryBlocks, 0, sizeof(GrabbedMemoryBlocks));
		}
	};
#endif //USE_OS_SMALL_BLOCK_GRAB_MEMORY_FROM_OS
#endif //USE_OS_SMALL_BLOCK_ALLOC
	
	// Implementation. 
	static CA_NO_RETURN void OutOfMemory(uint64 Size, uint32 Alignment=0)
	{
		// this is expected not to return
		FPlatformMemory::OnOutOfMemory(Size, Alignment);
	}

	static FORCEINLINE void TrackStats(FPoolTable* Table, uint32 Size)
	{
#if STATS
		// keep track of memory lost to padding
		Table->TotalWaste += Table->BlockSize - Size;
		Table->TotalRequests++;
		Table->ActiveRequests++;
		Table->MaxActiveRequests = FMath::Max(Table->MaxActiveRequests, Table->ActiveRequests);
		Table->MaxRequest = Size > Table->MaxRequest ? Size : Table->MaxRequest;
		Table->MinRequest = Size < Table->MinRequest ? Size : Table->MinRequest;
#endif
	}

	/** 
	 * Create a 64k page of FPoolInfo structures for tracking allocations
	 */
	static FPoolInfo* CreateIndirect(FMallocBinned& Allocator)
	{
		uint64 IndirectPoolBlockSizeBytes = Allocator.IndirectPoolBlockSize * sizeof(FPoolInfo);

		checkSlow(IndirectPoolBlockSizeBytes <= Allocator.PageSize);
        LLM_PLATFORM_SCOPE(ELLMTag::FMalloc);
		FPoolInfo* Indirect = (FPoolInfo*)FPlatformMemory::BinnedAllocFromOS(IndirectPoolBlockSizeBytes);
		if( !Indirect )
		{
			OutOfMemory(IndirectPoolBlockSizeBytes);
		}
		FMemory::Memset(Indirect, 0, IndirectPoolBlockSizeBytes);

		BINNED_PEAK_STATCOUNTER(Allocator.OsPeak,    BINNED_ADD_STATCOUNTER(Allocator.OsCurrent,    (int64)(Align(IndirectPoolBlockSizeBytes, Allocator.PageSize))));
		BINNED_PEAK_STATCOUNTER(Allocator.WastePeak, BINNED_ADD_STATCOUNTER(Allocator.WasteCurrent, (int64)(Align(IndirectPoolBlockSizeBytes, Allocator.PageSize))));

		return Indirect;
	}

	/**
	* Initializes tables for HashBuckets if they haven't already been initialized.
	*/
	static FORCEINLINE PoolHashBucket* CreateHashBuckets(const FMallocBinned& Allocator)
	{
		// Init tables.
		LLM_PLATFORM_SCOPE(ELLMTag::FMalloc);
		PoolHashBucket* Result = (PoolHashBucket*)FPlatformMemory::BinnedAllocFromOS(Align(Allocator.MaxHashBuckets * sizeof(PoolHashBucket), Allocator.PageSize));

		for (uint32 i = 0; i < Allocator.MaxHashBuckets; ++i)
		{
			new (Result + i) PoolHashBucket();
		}

		return Result;
	}
	
	/** 
	* Gets the FPoolInfo for a memory address. If no valid info exists one is created. 
	* NOTE: This function requires a mutex across threads, but its is the callers responsibility to 
	* acquire the mutex before calling
	*/
	static FORCEINLINE FPoolInfo* GetPoolInfo(FMallocBinned& Allocator, UPTRINT Ptr)
	{
		if (!Allocator.HashBuckets)
		{
			Allocator.HashBuckets = CreateHashBuckets(Allocator);
		}
		checkSlow(Allocator.HashBuckets);

		UPTRINT Key       = Ptr >> Allocator.HashKeyShift;
		UPTRINT Hash      = Key & (Allocator.MaxHashBuckets - 1);
		UPTRINT PoolIndex = ((UPTRINT)Ptr >> Allocator.PoolBitShift) & Allocator.PoolMask;

		PoolHashBucket* Collision = &Allocator.HashBuckets[Hash];
		do
		{
			if (Collision->Key == Key || !Collision->FirstPool)
			{
				if (!Collision->FirstPool)
				{
					Collision->Key = Key;
					InitializeHashBucket(Allocator, Collision);
					CA_ASSUME(Collision->FirstPool);
				}
				return &Collision->FirstPool[PoolIndex];
			}

			Collision = Collision->Next;
		} while (Collision != &Allocator.HashBuckets[Hash]);

		//Create a new hash bucket entry
		PoolHashBucket* NewBucket = CreateHashBucket(Allocator);
		NewBucket->Key = Key;
		Allocator.HashBuckets[Hash].Link(NewBucket);

		return &NewBucket->FirstPool[PoolIndex];
	}

	static FORCEINLINE FPoolInfo* FindPoolInfo(FMallocBinned& Allocator, UPTRINT Ptr1, UPTRINT& AllocationBase)
	{
		uint16  NextStep = 0;
		UPTRINT Ptr      = Ptr1 &~ ((UPTRINT)Allocator.PageSize - 1);
		for (uint32 i = 0, n = (BINNED_ALLOC_POOL_SIZE / Allocator.PageSize) + 1; i < n; ++i)
		{
			FPoolInfo* Pool = FindPoolInfoInternal(Allocator, Ptr, NextStep);
			if (Pool)
			{
				AllocationBase = Ptr;
				//checkSlow(Ptr1 >= AllocationBase && Ptr1 < AllocationBase + Pool->GetBytes());
				return Pool;
			}
			Ptr = ((Ptr - (Allocator.PageSize * NextStep)) - 1) &~ ((UPTRINT)Allocator.PageSize - 1);
		}
		AllocationBase = 0;
		return nullptr;
	}

	static FORCEINLINE FPoolInfo* FindPoolInfoInternal(FMallocBinned& Allocator, UPTRINT Ptr, uint16& JumpOffset)
	{
		checkSlow(Allocator.HashBuckets);

		uint32 Key       = (uint32)(Ptr >> Allocator.HashKeyShift);
		uint32 Hash      = Key & (Allocator.MaxHashBuckets - 1);
		uint32 PoolIndex = (uint32)(((UPTRINT)Ptr >> Allocator.PoolBitShift) & Allocator.PoolMask);

		JumpOffset = 0;

		PoolHashBucket* Collision = &Allocator.HashBuckets[Hash];
		do
		{
			if (Collision->Key==Key)
			{
				if (!Collision->FirstPool[PoolIndex].AllocSize)
				{
					JumpOffset = Collision->FirstPool[PoolIndex].TableIndex;
					return nullptr;
				}
				return &Collision->FirstPool[PoolIndex];
			}
			Collision = Collision->Next;
		} while (Collision != &Allocator.HashBuckets[Hash]);

		return nullptr;
	}

	/**
	 *	Returns a newly created and initialized PoolHashBucket for use.
	 */
	static FORCEINLINE PoolHashBucket* CreateHashBucket(FMallocBinned& Allocator) 
	{
		PoolHashBucket* Bucket = AllocateHashBucket(Allocator);
		InitializeHashBucket(Allocator, Bucket);
		return Bucket;
	}

	/**
	 *	Initializes bucket with valid parameters
	 *	@param bucket pointer to be initialized
	 */
	static FORCEINLINE void InitializeHashBucket(FMallocBinned& Allocator, PoolHashBucket* Bucket)
	{
		if (!Bucket->FirstPool)
		{
			Bucket->FirstPool = CreateIndirect(Allocator);
		}
	}

	/**
	 * Allocates a hash bucket from the free list of hash buckets
	*/
	static PoolHashBucket* AllocateHashBucket(FMallocBinned& Allocator)
	{
		if (!Allocator.HashBucketFreeList) 
		{
			const uint32 PageSize = Allocator.PageSize;

            LLM_PLATFORM_SCOPE(ELLMTag::FMalloc);
			Allocator.HashBucketFreeList = (PoolHashBucket*)FPlatformMemory::BinnedAllocFromOS(PageSize);
			BINNED_PEAK_STATCOUNTER(Allocator.OsPeak,    BINNED_ADD_STATCOUNTER(Allocator.OsCurrent,    PageSize));
			BINNED_PEAK_STATCOUNTER(Allocator.WastePeak, BINNED_ADD_STATCOUNTER(Allocator.WasteCurrent, PageSize));
			
			for (UPTRINT i = 0, n = PageSize / sizeof(PoolHashBucket); i < n; ++i)
			{
				Allocator.HashBucketFreeList->Link(new (Allocator.HashBucketFreeList + i) PoolHashBucket());
			}
		}

		PoolHashBucket* NextFree = Allocator.HashBucketFreeList->Next;
		PoolHashBucket* Free     = Allocator.HashBucketFreeList;

		Free->Unlink();
		if (NextFree == Free) 
		{
			NextFree = nullptr;
		}
		Allocator.HashBucketFreeList = NextFree;

		return Free;
	}

	static FPoolInfo* AllocatePoolMemory(FMallocBinned& Allocator, FPoolTable* Table, uint32 PoolSize, uint16 TableIndex)
	{
		const uint32 PageSize = Allocator.PageSize;

		// Must create a new pool.
		uint32 Blocks   = PoolSize / Table->BlockSize;
		uint32 Bytes    = Blocks * Table->BlockSize;
		UPTRINT OsBytes = Align(Bytes, PageSize);

		checkSlow(Blocks >= 1);
		checkSlow(PoolSize >= Bytes);
		
		FFreeMem* Free = nullptr;
		SIZE_T ActualPoolSize; //TODO: use this to reduce waste?
		
#if USE_OS_SMALL_BLOCK_GRAB_MEMORY_FROM_OS
		Free = GetAllocFromSmallBlockGrab(Allocator, OsBytes);
#endif //USE_OS_SMALL_BLOCK_GRAB_MEMORY_FROM_OS
		
		// Allocate memory if we haven't yet
		if (Free == nullptr)
		{
			Free = (FFreeMem*)OSAlloc(Allocator, OsBytes, ActualPoolSize);
		}
		
        checkSlow(IsAligned(Free, PageSize));
		if( !Free )
		{
			OutOfMemory(OsBytes);
		}

		// Create pool in the indirect table.
		FPoolInfo* Pool;
		{
#ifdef USE_FINE_GRAIN_LOCKS
			UE::TUniqueLock PoolInfoLock(Allocator.AccessGuard);
#endif
			Pool = GetPoolInfo(Allocator, (UPTRINT)Free);
			for (UPTRINT i = (UPTRINT)PageSize, Offset = 0; i < OsBytes; i += PageSize, ++Offset)
			{
				FPoolInfo* TrailingPool = GetPoolInfo(Allocator, ((UPTRINT)Free) + i);
				check(TrailingPool);

				//Set trailing pools to point back to first pool
				TrailingPool->SetAllocationSizes(0, 0, (uint32)Offset, (uint32)Allocator.BinnedOSTableIndex);
			}

			
			BINNED_PEAK_STATCOUNTER(Allocator.OsPeak,    BINNED_ADD_STATCOUNTER(Allocator.OsCurrent,    OsBytes));
			BINNED_PEAK_STATCOUNTER(Allocator.WastePeak, BINNED_ADD_STATCOUNTER(Allocator.WasteCurrent, (OsBytes - Bytes)));
		}

		// Init pool.
		Pool->Link( Table->FirstPool );
		Pool->SetAllocationSizes(Bytes, OsBytes, TableIndex, Allocator.BinnedOSTableIndex);
		Pool->Taken		 = 0;
		Pool->FirstMem   = Free;

#if STATS
		Table->NumActivePools++;
		Table->MaxActivePools = FMath::Max(Table->MaxActivePools, Table->NumActivePools);
#endif
		// Create first free item.
		Free->NumFreeBlocks = Blocks;
		Free->Next          = nullptr;

		return Pool;
	}

	static FORCEINLINE FFreeMem* AllocateBlockFromPool(FMallocBinned& Allocator, FPoolTable* Table, FPoolInfo* Pool, uint32 Alignment)
	{
		// Pick first available block and unlink it.
		Pool->Taken++;
		checkSlow(Pool->TableIndex < Allocator.BinnedOSTableIndex); // if this is false, FirstMem is actually a size not a pointer
		checkSlow(Pool->FirstMem);
		checkSlow(Pool->FirstMem->NumFreeBlocks > 0);
		checkSlow(Pool->FirstMem->NumFreeBlocks < PAGE_SIZE_LIMIT);
		FFreeMem* Free = (FFreeMem*)((uint8*)Pool->FirstMem + --Pool->FirstMem->NumFreeBlocks * Table->BlockSize);
		if( !Pool->FirstMem->NumFreeBlocks )
		{
			Pool->FirstMem = Pool->FirstMem->Next;
			if( !Pool->FirstMem )
			{
				// Move to exhausted list.
				Pool->Unlink();
				Pool->Link( Table->ExhaustedPool );
			}
		}
		BINNED_PEAK_STATCOUNTER(Allocator.UsedPeak, BINNED_ADD_STATCOUNTER(Allocator.UsedCurrent, Table->BlockSize));
		return Align(Free, Alignment);
	}

	/**
	* Releases memory back to the system. This is not protected from multi-threaded access and it's
	* the callers responsibility to Lock AccessGuard before calling this.
	*/
	static void FreeInternal(FMallocBinned& Allocator, void* Ptr)
	{
		MEM_TIME(MemTime -= FPlatformTime::Seconds());
		BINNED_DECREMENT_STATCOUNTER(Allocator.CurrentAllocs);

#if PLATFORM_IOS || PLATFORM_MAC
		if(FPlatformMemory::PtrIsOSMalloc(Ptr))
		{
			SmallOSFree(Allocator, Ptr, Private::SMALL_BLOCK_POOL_SIZE);
			return;
		}
#endif
		UPTRINT BasePtr;
		FPoolInfo* Pool = FindPoolInfo(Allocator, (UPTRINT)Ptr, BasePtr);
		checkSlow(Pool);
		checkSlow(Pool->GetBytes() != 0);
		if (Pool->TableIndex < Allocator.BinnedOSTableIndex)
		{
			FPoolTable* Table = Allocator.MemSizeToPoolTable[Pool->TableIndex];
#ifdef USE_FINE_GRAIN_LOCKS
			UE::TUniqueLock TableLock(Table->CriticalSection);
#endif
#if STATS
			Table->ActiveRequests--;
#endif
			// If this pool was exhausted, move to available list.
			if( !Pool->FirstMem )
			{
				Pool->Unlink();
				Pool->Link( Table->FirstPool );
			}

			void* BaseAddress = (void*)BasePtr;
			uint32 BlockSize = Table->BlockSize;
			PTRINT OffsetFromBase = (PTRINT)Ptr - (PTRINT)BaseAddress;
			check(OffsetFromBase >= 0);
			uint32 AlignOffset = OffsetFromBase % BlockSize;

			// Patch pointer to include previously applied alignment.
			Ptr = (void*)((PTRINT)Ptr - (PTRINT)AlignOffset);

			// Free a pooled allocation.
			FFreeMem* Free		= (FFreeMem*)Ptr;
			Free->NumFreeBlocks	= 1;
			Free->Next			= Pool->FirstMem;
			Pool->FirstMem		= Free;
			BINNED_ADD_STATCOUNTER(Allocator.UsedCurrent, -(int64)(Table->BlockSize));

			// Free this pool.
			checkSlow(Pool->Taken >= 1);
			if( --Pool->Taken == 0 )
			{
#if STATS
				Table->NumActivePools--;
#endif
				// Free the OS memory.
				SIZE_T OsBytes = Pool->GetOsBytes(Allocator.PageSize, Allocator.BinnedOSTableIndex);
				BINNED_ADD_STATCOUNTER(Allocator.OsCurrent,    -(int64)OsBytes);
				BINNED_ADD_STATCOUNTER(Allocator.WasteCurrent, -(int64)(OsBytes - Pool->GetBytes()));
				Pool->Unlink();
				Pool->SetAllocationSizes(0, 0, 0, Allocator.BinnedOSTableIndex);
				OSFree(Allocator, (void*)BasePtr, OsBytes);
			}
		}
		else
		{
			// Free an OS allocation.
			checkSlow(IsAligned(Ptr, Allocator.PageSize));
			SIZE_T OsBytes = Pool->GetOsBytes(Allocator.PageSize, Allocator.BinnedOSTableIndex);

			BINNED_ADD_STATCOUNTER(Allocator.UsedCurrent,  -(int64)Pool->GetBytes());
			BINNED_ADD_STATCOUNTER(Allocator.OsCurrent,    -(int64)OsBytes);
			BINNED_ADD_STATCOUNTER(Allocator.WasteCurrent, -(int64)(OsBytes - Pool->GetBytes()));
			OSFree(Allocator, (void*)BasePtr, OsBytes);
		}

		MEM_TIME(MemTime += FPlatformTime::Seconds());
	}

	static void PushFreeLockless(FMallocBinned& Allocator, void* Ptr)
	{
#ifdef USE_LOCKFREE_DELETE
		Allocator.PendingFreeList->Push(Ptr);
#else
#ifdef USE_COARSE_GRAIN_LOCKS
		UE::TUniqueLock ScopedLock(Allocator.AccessGuard);
#endif
		FreeInternal(Allocator, Ptr);
#endif
	}

	/**
	* Clear and Process the list of frees to be deallocated. It's the callers
	* responsibility to Lock AccessGuard before calling this
	*/
	static void FlushPendingFrees(FMallocBinned& Allocator)
	{
#ifdef USE_LOCKFREE_DELETE
		if (!Allocator.PendingFreeList && !Allocator.bDoneFreeListInit)
		{
			Allocator.bDoneFreeListInit = true;
			Allocator.PendingFreeList = new ((void*)Allocator.PendingFreeListMemory) TLockFreePointerList<void>();
		}

		// Because a lockless list and TArray calls new/malloc internally, need to guard against re-entry
		if (Allocator.bFlushingFrees || !Allocator.PendingFreeList)
		{
			return;
		}
		Allocator.bFlushingFrees = true;
		Allocator.PendingFreeList->PopAll(Allocator.FlushedFrees);
		for (uint32 i = 0, n = Allocator.FlushedFrees.Num(); i < n; ++i)
		{
			FreeInternal(Allocator, Allocator.FlushedFrees[i]);
		}
		Allocator.FlushedFrees.Reset();
		Allocator.bFlushingFrees = false;
#endif
	}

	static FORCEINLINE void OSFree(FMallocBinned& Allocator, void* Ptr, SIZE_T Size)
	{
#ifdef CACHE_FREED_OS_ALLOCS
#ifdef USE_FINE_GRAIN_LOCKS
		UE::TUniqueLock MainLock(Allocator.AccessGuard);
#endif
		if (Size > MAX_CACHED_OS_FREES_BYTE_LIMIT / 4)
		{
			#if USE_OS_SMALL_BLOCK_GRAB_MEMORY_FROM_OS
				if (IsSmallBlockGrabAllocation(ptr))
				{
					FreeSmallBlockGrab(ptr, Allocator, Size);
				}
				else
			#endif //USE_OS_SMALL_BLOCK_GRAB_MEMORY_FROM_OS
			{
				FPlatformMemory::BinnedFreeToOS(Ptr, Size);
			}
			return;
		}

		while (Allocator.FreedPageBlocksNum && (Allocator.FreedPageBlocksNum >= MAX_CACHED_OS_FREES || Allocator.CachedTotal + Size > MAX_CACHED_OS_FREES_BYTE_LIMIT)) 
		{
			//Remove the oldest one
			void* FreePtr = Allocator.FreedPageBlocks[0].Ptr;
			SIZE_T FreeSize = Allocator.FreedPageBlocks[0].ByteSize;
			Allocator.CachedTotal -= FreeSize;
			Allocator.FreedPageBlocksNum--;
			if (Allocator.FreedPageBlocksNum)
			{
				FMemory::Memmove(&Allocator.FreedPageBlocks[0], &Allocator.FreedPageBlocks[1], sizeof(FFreePageBlock) * Allocator.FreedPageBlocksNum);
			}
			
			#if USE_OS_SMALL_BLOCK_GRAB_MEMORY_FROM_OS
				if (IsSmallBlockGrabAllocation(FreePtr))
				{
					FreeSmallBlockGrab(FreePtr, Allocator, Size);
				}
				else
			#endif //USE_OS_SMALL_BLOCK_GRAB_MEMORY_FROM_OS
			{
				FPlatformMemory::BinnedFreeToOS(FreePtr, FreeSize);
			}
		}
		Allocator.FreedPageBlocks[Allocator.FreedPageBlocksNum].Ptr      = Ptr;
		Allocator.FreedPageBlocks[Allocator.FreedPageBlocksNum].ByteSize = Size;
		Allocator.CachedTotal += Size;
		++Allocator.FreedPageBlocksNum;
#else
		#if USE_OS_SMALL_BLOCK_GRAB_MEMORY_FROM_OS
			if (Private::IsSmallBlockGrabAllocation(Ptr))
			{
				FreeSmallBlockGrab(Ptr, Allocator, Size);
			}
			else
		#endif //USE_OS_SMALL_BLOCK_GRAB_MEMORY_FROM_OS
		{
			FPlatformMemory::BinnedFreeToOS(Ptr, Size);
		}
#endif
	}

	static FORCEINLINE void SmallOSFree(FMallocBinned& Allocator, void* Ptr, SIZE_T Size)
	{
#if PLATFORM_IOS
		LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelFree(ELLMTracker::Platform, Ptr));
		::free(Ptr);
		Allocator.bNanoMallocAvailable = true;
#else
		FPlatformMemory::BinnedFreeToOS(Ptr, Size);
#endif
	}
	
	static FORCEINLINE void* OSAlloc(FMallocBinned& Allocator, SIZE_T NewSize, SIZE_T& OutActualSize)
	{
#ifdef CACHE_FREED_OS_ALLOCS
		{
#ifdef USE_FINE_GRAIN_LOCKS
			// We want to hold the lock a little as possible so release it
			// before the big call to the OS
			UE::TUniqueLock MainLock(Allocator.AccessGuard);
#endif
			for (uint32 i=0; i < Allocator.FreedPageBlocksNum; ++i)
			{
				// look for exact matches first, these are aligned to the page size, so it should be quite common to hit these on small pages sizes
				if (Allocator.FreedPageBlocks[i].ByteSize == NewSize)
				{
					void* Ret = Allocator.FreedPageBlocks[i].Ptr;
					UE_CLOGF(!Ret, LogMemory, Fatal, "OS memory allocation cache has been corrupted!");
					OutActualSize = Allocator.FreedPageBlocks[i].ByteSize;
					Allocator.CachedTotal -= Allocator.FreedPageBlocks[i].ByteSize;
					if (i < Allocator.FreedPageBlocksNum - 1)
					{
						FMemory::Memmove(&Allocator.FreedPageBlocks[i], &Allocator.FreedPageBlocks[i + 1], sizeof(FFreePageBlock) * (Allocator.FreedPageBlocksNum - i - 1));
					}
					Allocator.FreedPageBlocksNum--;
					return Ret;
				}
			};
			// Note the code below was similar to code removed by Arciel in his MallocBinned2 update and its removal fixes a memory leak
/*			for (uint32 i=0; i < Allocator.FreedPageBlocksNum; ++i)
			{
				// is it possible (and worth i.e. <25% overhead) to use this block
				if (Allocator.FreedPageBlocks[i].ByteSize >= NewSize && Allocator.FreedPageBlocks[i].ByteSize * 3 <= NewSize * 4)
				{
					void* Ret = Allocator.FreedPageBlocks[i].Ptr;
					UE_CLOGF(!Ret, LogMemory, Fatal, "OS memory allocation cache has been corrupted!");
					OutActualSize = Allocator.FreedPageBlocks[i].ByteSize;
					Allocator.CachedTotal -= Allocator.FreedPageBlocks[i].ByteSize;
					if (i < Allocator.FreedPageBlocksNum - 1)
					{
						FMemory::Memmove(&Allocator.FreedPageBlocks[i], &Allocator.FreedPageBlocks[i + 1], sizeof(FFreePageBlock) * (Allocator.FreedPageBlocksNum - i - 1));
					}
					Allocator.FreedPageBlocksNum--;
					return Ret;
				}
			};*/
		}
		OutActualSize = NewSize;
        LLM_PLATFORM_SCOPE(ELLMTag::SmallBinnedAllocation);
		void* Ptr = FPlatformMemory::BinnedAllocFromOS(NewSize);
		if (!Ptr)
		{
			//Are we holding on to much mem? Release it all.
			FlushAllocCache(Allocator);
            LLM_PLATFORM_SCOPE(ELLMTag::SmallBinnedAllocation);
			Ptr = FPlatformMemory::BinnedAllocFromOS(NewSize);
		}
		return Ptr;
#else
		(void)OutActualSize;
        LLM_PLATFORM_SCOPE(ELLMTag::FMalloc);
		return FPlatformMemory::BinnedAllocFromOS(NewSize);
#endif
	}

	static FORCEINLINE void* SmallOSAlloc(FMallocBinned& Allocator, SIZE_T NewSize, SIZE_T& OutActualSize)
	{
#if PLATFORM_IOS
		(void)OutActualSize;
		LLM_PLATFORM_SCOPE(ELLMTag::FMalloc);
		void* Ptr = ::malloc(NewSize);
		if (Ptr == nullptr)
		{
			UE_LOGF(LogTemp, Warning, "malloc failure allocating %zu, error code: %d", NewSize, errno);
		}
		LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelAlloc(ELLMTracker::Platform, Ptr, NewSize));
		return Ptr;
#else
		(void)OutActualSize;
		LLM_PLATFORM_SCOPE(ELLMTag::FMalloc);
		return FPlatformMemory::BinnedAllocFromOS(NewSize);
#endif
	}
	
#ifdef CACHE_FREED_OS_ALLOCS
	static void FlushAllocCache(FMallocBinned& Allocator)
	{
#ifdef USE_FINE_GRAIN_LOCKS
		UE::TUniqueLock MainLock(Allocator.AccessGuard);
#endif
		for (int i = 0, n = Allocator.FreedPageBlocksNum; i<n; ++i) 
		{
			//Remove allocs
			FPlatformMemory::BinnedFreeToOS(Allocator.FreedPageBlocks[i].Ptr, Allocator.FreedPageBlocks[i].ByteSize);
			Allocator.FreedPageBlocks[i].Ptr = nullptr;
			Allocator.FreedPageBlocks[i].ByteSize = 0;
		}
		Allocator.FreedPageBlocksNum = 0;
		Allocator.CachedTotal        = 0;
	}
#endif

	static void UpdateSlackStat(FMallocBinned& Allocator)
	{
	#if	STATS
		SIZE_T LocalWaste = Allocator.WasteCurrent;
		double Waste = 0.0;
		for( int32 PoolIndex = 0; PoolIndex < POOL_COUNT; PoolIndex++ )
		{
			FPoolTable& Table = Allocator.PoolTable[PoolIndex];

			Waste += ( ( double )Table.TotalWaste / ( double )Table.TotalRequests ) * ( double )Table.ActiveRequests;
			Waste += Table.NumActivePools * ( BINNED_ALLOC_POOL_SIZE - ( ( BINNED_ALLOC_POOL_SIZE / Table.BlockSize ) * Table.BlockSize ) );
		}
		LocalWaste += ( uint32 )Waste;
		Allocator.SlackCurrent = Allocator.OsCurrent - LocalWaste - Allocator.UsedCurrent; //-V567
	#endif // STATS
	}
};

#if USE_OS_SMALL_BLOCK_GRAB_MEMORY_FROM_OS
uint64 FMallocBinned::Private::SmallBlockGrab_FreeStartPointers[MAX_NUM_BLOCK_START_ADDRESSES];
int FMallocBinned::Private::NumFreeSmallBlockGrabAllocations = 0;
#endif

bool FMallocBinned::Private::bHasInitializedStatsMetadata = false;

void FMallocBinned::GetAllocatorStats( FGenericMemoryStats& out_Stats )
{
	FMalloc::GetAllocatorStats( out_Stats );

	if (!Private::bHasInitializedStatsMetadata)
	{
		InitializeStatsMetadata();
	}
	
#if	STATS
	SIZE_T	LocalOsCurrent = 0;
	SIZE_T	LocalOsPeak = 0;
	SIZE_T	LocalWasteCurrent = 0;
	SIZE_T	LocalWastePeak = 0;
	SIZE_T	LocalUsedCurrent = 0;
	SIZE_T	LocalUsedPeak = 0;
	SIZE_T	LocalCurrentAllocs = 0;
	SIZE_T	LocalTotalAllocs = 0;
	SIZE_T	LocalSlackCurrent = 0;
	
#if USE_OS_SMALL_BLOCK_GRAB_MEMORY_FROM_OS
	SIZE_T LocalNanoMallocPagesCurrent;
	SIZE_T LocalNanoMallocPagesPeak;
	SIZE_T LocalNanoMallocPagesWaste;
	SIZE_T LocalNanoMallocWastePagesPeak;
#endif //USE_OS_SMALL_BLOCK_GRAB_MEMORY_FROM_OS
	
	{
#ifdef USE_INTERNAL_LOCKS
		UE::TUniqueLock ScopedLock(AccessGuard);
#endif

		Private::UpdateSlackStat(*this);

		// Copy memory stats.
		LocalOsCurrent = OsCurrent;
		LocalOsPeak = OsPeak;
		LocalWasteCurrent = WasteCurrent;
		LocalWastePeak = WastePeak;
		LocalUsedCurrent = UsedCurrent;
		LocalUsedPeak = UsedPeak;
		LocalCurrentAllocs = CurrentAllocs;
		LocalTotalAllocs = TotalAllocs;
		LocalSlackCurrent = SlackCurrent;

#if USE_OS_SMALL_BLOCK_GRAB_MEMORY_FROM_OS
		LocalNanoMallocPagesCurrent = NanoMallocPagesCurrent;
		LocalNanoMallocPagesPeak = NanoMallocPagesPeak;
		LocalNanoMallocPagesWaste = NanoMallocPagesWaste;
		LocalNanoMallocWastePagesPeak = NanoMallocWastePagesPeak;
#endif //USE_OS_SMALL_BLOCK_GRAB_MEMORY_FROM_OS
	}

	// Malloc binned stats.
	out_Stats.Add( GET_STATDESCRIPTION( STAT_Binned_OsCurrent ), LocalOsCurrent );
	out_Stats.Add( GET_STATDESCRIPTION( STAT_Binned_OsPeak ), LocalOsPeak );
	out_Stats.Add( GET_STATDESCRIPTION( STAT_Binned_WasteCurrent ), LocalWasteCurrent );
	out_Stats.Add( GET_STATDESCRIPTION( STAT_Binned_WastePeak ), LocalWastePeak );
	out_Stats.Add( GET_STATDESCRIPTION( STAT_Binned_UsedCurrent ), LocalUsedCurrent );
	out_Stats.Add( GET_STATDESCRIPTION( STAT_Binned_UsedPeak ), LocalUsedPeak );
	out_Stats.Add( GET_STATDESCRIPTION( STAT_Binned_CurrentAllocs ), LocalCurrentAllocs );
	out_Stats.Add( GET_STATDESCRIPTION( STAT_Binned_TotalAllocs ), LocalTotalAllocs );
	out_Stats.Add( GET_STATDESCRIPTION( STAT_Binned_SlackCurrent ), LocalSlackCurrent );
	
#if USE_OS_SMALL_BLOCK_GRAB_MEMORY_FROM_OS && ENABLE_LOW_LEVEL_MEM_TRACKER
	out_Stats.Add( GET_STATDESCRIPTION( STAT_Binned_NanoMallocPages_Current ), LocalNanoMallocPagesCurrent );
	out_Stats.Add( GET_STATDESCRIPTION( STAT_Binned_NanoMallocPages_Peak ), LocalNanoMallocPagesPeak );
	out_Stats.Add( GET_STATDESCRIPTION( STAT_Binned_NanoMallocPages_WasteCurrent ), LocalNanoMallocPagesWaste );
	out_Stats.Add( GET_STATDESCRIPTION( STAT_Binned_NanoMallocPages_WastePeak ), LocalNanoMallocWastePagesPeak );
#endif //USE_OS_SMALL_BLOCK_GRAB_MEMORY_FROM_OS
	
#endif // STATS
}

void FMallocBinned::InitializeStatsMetadata()
{
	Private::bHasInitializedStatsMetadata = true;
	
	FMalloc::InitializeStatsMetadata();

	// Initialize stats metadata here instead of UpdateStats.
	// Mostly to avoid dead-lock when stats malloc profiler is enabled.
	GET_STATFNAME(STAT_Binned_OsCurrent);
	GET_STATFNAME(STAT_Binned_OsPeak);
	GET_STATFNAME(STAT_Binned_WasteCurrent);
	GET_STATFNAME(STAT_Binned_WastePeak);
	GET_STATFNAME(STAT_Binned_UsedCurrent);
	GET_STATFNAME(STAT_Binned_UsedPeak);
	GET_STATFNAME(STAT_Binned_CurrentAllocs);
	GET_STATFNAME(STAT_Binned_TotalAllocs);
	GET_STATFNAME(STAT_Binned_SlackCurrent);
	
#if USE_OS_SMALL_BLOCK_GRAB_MEMORY_FROM_OS && ENABLE_LOW_LEVEL_MEM_TRACKER
	GET_STATFNAME(STAT_Binned_NanoMallocPages_Current);
	GET_STATFNAME(STAT_Binned_NanoMallocPages_Peak);
	GET_STATFNAME(STAT_Binned_NanoMallocPages_WasteCurrent);
	GET_STATFNAME(STAT_Binned_NanoMallocPages_WastePeak);
#endif //USE_OS_SMALL_BLOCK_GRAB_MEMORY_FROM_OS
}

FMallocBinned::FMallocBinned(uint32 InPageSize, uint64 AddressLimit)
	:	TableAddressLimit(AddressLimit)
#ifdef USE_LOCKFREE_DELETE
	,	PendingFreeList(nullptr)
	,	bFlushingFrees(false)
	,	bDoneFreeListInit(false)
#endif
	,	HashBuckets(nullptr)
	,	HashBucketFreeList(nullptr)
	,	PageSize		(InPageSize)
#ifdef CACHE_FREED_OS_ALLOCS
	,	FreedPageBlocksNum(0)
	,	CachedTotal(0)
#endif
#if STATS
	,	OsCurrent		( 0 )
	,	OsPeak			( 0 )
	,	WasteCurrent	( 0 )
	,	WastePeak		( 0 )
	,	UsedCurrent		( 0 )
	,	UsedPeak		( 0 )
	,	CurrentAllocs	( 0 )
	,	TotalAllocs		( 0 )
	,	SlackCurrent	( 0 )
	,	MemTime			( 0.0 )
#if USE_OS_SMALL_BLOCK_GRAB_MEMORY_FROM_OS
	,	NanoMallocPagesCurrent 		( 0 )
	,	NanoMallocPagesPeak			( 0 )
	,	NanoMallocPagesWaste		( 0 )
	,	NanoMallocWastePagesPeak	( 0 )
#endif //USE_OS_SMALL_BLOCK_GRAB_MEMORY_FROM_OS
#endif
{
	check(!(PageSize & (PageSize - 1)));
	check(!(AddressLimit & (AddressLimit - 1)));
	check(PageSize <= 65536); // There is internal limit on page size of 64k
	check(AddressLimit > PageSize); // Check to catch 32 bit overflow in AddressLimit

#if USE_OS_SMALL_BLOCK_ALLOC
	
	FPlatformMemory::NanoMallocInit();
	bNanoMallocAvailable = FPlatformMemory::IsNanoMallocAvailable();
	
#if USE_OS_SMALL_BLOCK_GRAB_MEMORY_FROM_OS

	void* GrabberHelperLoc = ::malloc(sizeof(Private::FMemoryAllocationGrabberHelper));
	Private::FMemoryAllocationGrabberHelper* GrabberHelper = new (GrabberHelperLoc) Private::FMemoryAllocationGrabberHelper();
	
	//Grab as much as we can in smaller and smaller amounts until we can get no more
	int CurrentAllocationSize = Private::SMALL_BLOCK_GRAB_MAX_ALLOC_SIZE;
	int CurrentGrabIndex = 0;
	while ((CurrentAllocationSize >= Private::SMALL_BLOCK_GRAB_MIN_ALLOC_SIZE))
	{
		void* NewData = ::malloc(CurrentAllocationSize);
		if (((uint64)NewData) >= Private::SmallBlockStartPtr && ((uint64)NewData) < Private::SmallBlockEndPtr)
		{
			GrabberHelper->AddGrabbedMemory((uint64)NewData, CurrentAllocationSize);
		}
		else
		{
			//Free memory we just allocated as it was outside of the nano_malloc range
			::free(NewData);
			
			//Lower how much data we are trying to allocate now that 1 failed on this size
			CurrentAllocationSize -= Private::SMALL_BLOCK_GRAB_ALLOC_ALIGN;
		}
	}
	
	GrabberHelper->ValidateMemoryBlocks();
	
	GrabberHelper->FindGrabbedBlockSequentials(Private::SmallBlockGrab_FreeStartPointers);
	Private::NumFreeSmallBlockGrabAllocations = GrabberHelper->NumFoundCombinedBlocks;
	
	//Free our GrabberHelper now that we are fully allocated and good to go
	::free(GrabberHelperLoc);
	
#endif //USE_OS_SMALL_BLOCK_GRAB_MEMORY_FROM_OS
#endif //USE_OS_SMALL_BLOCK_ALLOC
	
	/** Shift to get the reference from the indirect tables */
	PoolBitShift = FPlatformMath::CeilLogTwo(PageSize);
	IndirectPoolBitShift = FPlatformMath::CeilLogTwo(PageSize/sizeof(FPoolInfo));
	IndirectPoolBlockSize = PageSize/sizeof(FPoolInfo);

	MaxHashBuckets = AddressLimit >> (IndirectPoolBitShift+PoolBitShift); 
	MaxHashBucketBits = FPlatformMath::CeilLogTwo(MaxHashBuckets);
	MaxHashBucketWaste = (MaxHashBuckets*sizeof(PoolHashBucket))/1024;
	MaxBookKeepingOverhead = ((AddressLimit/PageSize)*sizeof(PoolHashBucket))/(1024*1024);
	/** 
	* Shift required to get required hash table key.
	*/
	HashKeyShift = PoolBitShift+IndirectPoolBitShift;
	/** Used to mask off the bits that have been used to lookup the indirect table */
	PoolMask =  ( ( 1ull << ( HashKeyShift - PoolBitShift ) ) - 1 );
	BinnedSizeLimit = Private::PAGE_SIZE_LIMIT/2;
	BinnedOSTableIndex = BinnedSizeLimit+EXTENDED_PAGE_POOL_ALLOCATION_COUNT;

	check((BinnedSizeLimit & (BinnedSizeLimit-1)) == 0);


	// Init tables.
	OsTable.FirstPool = nullptr;
	OsTable.ExhaustedPool = nullptr;
	OsTable.BlockSize = 0;

	/** The following options are not valid for page sizes less than 64k. They are here to reduce waste*/
	PagePoolTable[0].FirstPool = nullptr;
	PagePoolTable[0].ExhaustedPool = nullptr;
	PagePoolTable[0].BlockSize = PageSize == Private::PAGE_SIZE_LIMIT ? BinnedSizeLimit+(BinnedSizeLimit/2) : 0;

	PagePoolTable[1].FirstPool = nullptr;
	PagePoolTable[1].ExhaustedPool = nullptr;
	PagePoolTable[1].BlockSize = PageSize == Private::PAGE_SIZE_LIMIT ? PageSize+BinnedSizeLimit : 0;

	// Block sizes are based around getting the maximum amount of allocations per pool, with as little alignment waste as possible.
	// Block sizes should be close to even divisors of the POOL_SIZE, and well distributed. They must be 16-byte aligned as well.
	static const uint32 BlockSizes[POOL_COUNT] =
	{
		16,		32,		48,		64,		80,		96,		112,	128,
		160,	192,	224,	256,	288,	320,	384,	448,
		512,	576,	640,	704,	768,	896,	1024,	1168,
		1360,	1632,	2048,	2336,	2720,	3264,	4096,	4672,
		5456,	6544,	8192,	9360,	10912,	13104,	16384,	21840,	32768
	};

	for( uint32 i = 0; i < POOL_COUNT; i++ )
	{
		PoolTable[i].FirstPool = nullptr;
		PoolTable[i].ExhaustedPool = nullptr;
		PoolTable[i].BlockSize = BlockSizes[i];
		check(IsAligned(BlockSizes[i], Private::DEFAULT_BINNED_ALLOCATOR_ALIGNMENT));
#if STATS
		PoolTable[i].MinRequest = PoolTable[i].BlockSize;
#endif
	}

	for( uint32 i=0; i<MAX_POOLED_ALLOCATION_SIZE; i++ )
	{
		uint32 Index = 0;
		while( PoolTable[Index].BlockSize < i )
		{
			++Index;
		}
		checkSlow(Index < POOL_COUNT);
		MemSizeToPoolTable[i] = &PoolTable[Index];
	}

	MemSizeToPoolTable[BinnedSizeLimit] = &PagePoolTable[0];
	MemSizeToPoolTable[BinnedSizeLimit+1] = &PagePoolTable[1];

	check(MAX_POOLED_ALLOCATION_SIZE - 1 == PoolTable[POOL_COUNT - 1].BlockSize);
}

FMallocBinned::~FMallocBinned()
{
}

bool FMallocBinned::IsInternallyThreadSafe() const
{ 
#ifdef USE_INTERNAL_LOCKS
	return true;
#else
	return false;
#endif
}

void* FMallocBinned::Malloc(SIZE_T Size, uint32 Alignment)
{
#ifdef USE_COARSE_GRAIN_LOCKS
	UE::TUniqueLock ScopedLock(AccessGuard);
#endif

	ensure(Alignment <= PageSize);

	Private::FlushPendingFrees(*this);

	MEM_TIME(MemTime -= FPlatformTime::Seconds());
	
	BINNED_INCREMENT_STATCOUNTER(CurrentAllocs);
	BINNED_INCREMENT_STATCOUNTER(TotalAllocs);

	Size = FMath::Max(Size, (SIZE_T)1);
	FFreeMem* Free = nullptr;
	bool bUsePools = true;
#if USE_OS_SMALL_BLOCK_ALLOC && !USE_OS_SMALL_BLOCK_GRAB_MEMORY_FROM_OS
	if (bNanoMallocAvailable && (Size <= Private::SMALL_BLOCK_POOL_SIZE) && (Alignment <= Private::DEFAULT_BINNED_ALLOCATOR_ALIGNMENT))
	{
		//Make sure we have initialized our hash buckets even if we are using the NANO_MALLOC grabber, as otherwise we can end
		//up making bad assumptions and trying to grab invalid data during a Realloc of this data.
		if (!HashBuckets)
		{
			HashBuckets = Private::CreateHashBuckets(*this);
		}

		bUsePools = false;
		SIZE_T ActualPoolSize; //TODO: use this to reduce waste?
		Free = (FFreeMem*)Private::SmallOSAlloc(*this, Size, ActualPoolSize);
		check(FPlatformMemory::PtrIsOSMalloc(Free));
		
		if(!FPlatformMemory::PtrIsFromNanoMalloc(Free))
		{
			// This means we've overflowed the nano zone's internal buckets, which are fixed
			// So we need to fall back to UE's allocator
			Private::SmallOSFree(*this, Free, Size);
			bUsePools = true;
			Free = nullptr;
			bNanoMallocAvailable = false;
		}
	}
#endif
	if (bUsePools)
	{
		Alignment = FMath::Max<uint32>(Alignment, Private::DEFAULT_BINNED_ALLOCATOR_ALIGNMENT);
		const uint32 PAGE_ALIGNMENT = 4096;

		if( (Size < BinnedSizeLimit) && (Alignment == Private::DEFAULT_BINNED_ALLOCATOR_ALIGNMENT) )
		{
			// Allocate from pool.
			FPoolTable* Table = MemSizeToPoolTable[Size];
#ifdef USE_FINE_GRAIN_LOCKS
			UE::TUniqueLock TableLock(Table->CriticalSection);
#endif
			Private::TrackStats(Table, (uint32)Size);

			FPoolInfo* Pool = Table->FirstPool;
			if( !Pool )
			{
				Pool = Private::AllocatePoolMemory(*this, Table, Private::BINNED_ALLOC_POOL_SIZE/*PageSize*/, Size);
			}

			Free = Private::AllocateBlockFromPool(*this, Table, Pool, Alignment);
		}
		else if ( ((Size >= BinnedSizeLimit && Size <= PagePoolTable[0].BlockSize) ||
				   (Size > PageSize && Size <= PagePoolTable[1].BlockSize)) && (Alignment <= PAGE_ALIGNMENT))
		{
			// Bucket in a pool of 3*PageSize or 6*PageSize
			uint32 BinType = Size < PageSize ? 0 : 1;
			uint32 PageCount = 3*BinType + 3;
			FPoolTable* Table = &PagePoolTable[BinType];
#ifdef USE_FINE_GRAIN_LOCKS
			UE::TUniqueLock TableLock(Table->CriticalSection);
#endif
			Private::TrackStats(Table, (uint32)Size);

			FPoolInfo* Pool = Table->FirstPool;
			if( !Pool )
			{
				Pool = Private::AllocatePoolMemory(*this, Table, PageCount*PageSize, BinnedSizeLimit+BinType);
			}

			Free = Private::AllocateBlockFromPool(*this, Table, Pool, Alignment);
		}
		else
		{
			// Use OS for large allocations.
			UPTRINT AlignedSize = Align(Size, PageSize);
			SIZE_T ActualPoolSize; //TODO: use this to reduce waste?
			Free = (FFreeMem*)Private::OSAlloc(*this, AlignedSize, ActualPoolSize);
			if (!Free)
			{
				Private::OutOfMemory(AlignedSize);
			}

			void* AlignedFree = Align(Free, Alignment);

			// Create indirect.
			FPoolInfo* Pool;
			{
#ifdef USE_FINE_GRAIN_LOCKS
				UE::TUniqueLock PoolInfoLock(AccessGuard);
#endif
				Pool = Private::GetPoolInfo(*this, (UPTRINT)Free);

				if ((UPTRINT)Free != ((UPTRINT)AlignedFree & ~((UPTRINT)PageSize - 1)))
				{
					// Mark the FPoolInfo for AlignedFree to jump back to the FPoolInfo for ptr.
					for (UPTRINT i = (UPTRINT)PageSize, Offset = 0; i < AlignedSize; i += PageSize, ++Offset)
					{
						FPoolInfo* TrailingPool = Private::GetPoolInfo(*this, ((UPTRINT)Free) + i);
						check(TrailingPool);
						//Set trailing pools to point back to first pool
						TrailingPool->SetAllocationSizes(0, 0, Offset, BinnedOSTableIndex);
					}
				}
			}
			Free = (FFreeMem*)AlignedFree;
			Pool->SetAllocationSizes(Size, AlignedSize, BinnedOSTableIndex, BinnedOSTableIndex);
			BINNED_PEAK_STATCOUNTER(OsPeak, BINNED_ADD_STATCOUNTER(OsCurrent, AlignedSize));
			BINNED_PEAK_STATCOUNTER(UsedPeak, BINNED_ADD_STATCOUNTER(UsedCurrent, Size));
			BINNED_PEAK_STATCOUNTER(WastePeak, BINNED_ADD_STATCOUNTER(WasteCurrent, (int64)(AlignedSize - Size)));
		}

#if USE_OS_SMALL_BLOCK_ALLOC
		check(!FPlatformMemory::PtrIsOSMalloc(Free));
#endif
	}

	MEM_TIME(MemTime += FPlatformTime::Seconds());
	return Free;
}

void* FMallocBinned::Realloc( void* Ptr, SIZE_T NewSize, uint32 Alignment )
{
	Alignment = FMath::Max<uint32>(Alignment, Private::DEFAULT_BINNED_ALLOCATOR_ALIGNMENT);
    ensure(Alignment <= PageSize);
	MEM_TIME(MemTime -= FPlatformTime::Seconds());
	UPTRINT BasePtr;
	void* NewPtr = Ptr;
	if( Ptr && NewSize )
	{
#if USE_OS_SMALL_BLOCK_ALLOC && !USE_OS_SMALL_BLOCK_GRAB_MEMORY_FROM_OS
		if(FPlatformMemory::PtrIsOSMalloc(Ptr))
		{
            bool bFallback = true;
            const SIZE_T OldSize = malloc_size(Ptr);
            if ((NewSize <= Private::SMALL_BLOCK_POOL_SIZE) && (Alignment <= Private::DEFAULT_BINNED_ALLOCATOR_ALIGNMENT))
            {
                LLM_PLATFORM_SCOPE(ELLMTag::FMalloc);
				LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelFree(ELLMTracker::Platform, Ptr));
                NewPtr = ::realloc(Ptr, NewSize);
				LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelAlloc(ELLMTracker::Platform, NewPtr, NewSize));
                bFallback = !FPlatformMemory::PtrIsFromNanoMalloc(NewPtr);
            }

            if (bFallback)
            {
                // We've overflowed the nano region
                // Fall back to UE's allocator
                Ptr = NewPtr;
                NewPtr = Malloc(NewSize, Alignment);
                FMemory::Memcpy(NewPtr, Ptr, FMath::Min(OldSize, NewSize));
                Private::SmallOSFree(*this, Ptr, OldSize);
            }
		}
		else
#endif
		{
			FPoolInfo* Pool = Private::FindPoolInfo(*this, (UPTRINT)Ptr, BasePtr);

			if( Pool->TableIndex < BinnedOSTableIndex )
			{
				// Allocated from pool, so grow or shrink if necessary.
				check(Pool->TableIndex > 0); // it isn't possible to allocate a size of 0, Malloc will increase the size to DEFAULT_BINNED_ALLOCATOR_ALIGNMENT
				if (NewSize > MemSizeToPoolTable[Pool->TableIndex]->BlockSize || NewSize <= MemSizeToPoolTable[Pool->TableIndex - 1]->BlockSize
					|| !IsAligned(Ptr, Alignment))		// If reallocation requires a pointer to have a different alignment than the original we need to allocate new mem and copy old over
				{
					NewPtr = Malloc(NewSize, Alignment);
					FMemory::Memcpy(NewPtr, Ptr, FMath::Min<SIZE_T>(NewSize, MemSizeToPoolTable[Pool->TableIndex]->BlockSize));
					Free( Ptr );
				}
			}
			else
			{
				// Allocated from OS.
				if( NewSize > Pool->GetOsBytes(PageSize, BinnedOSTableIndex) || NewSize * 3 < Pool->GetOsBytes(PageSize, BinnedOSTableIndex) * 2 )
				{
					// Grow or shrink.
					NewPtr = Malloc(NewSize, Alignment);
					FMemory::Memcpy(NewPtr, Ptr, FMath::Min<SIZE_T>(NewSize, Pool->GetBytes()));
					Free( Ptr );
				}
				else
				{
//need a lock to cover the SetAllocationSizes()
#ifdef USE_FINE_GRAIN_LOCKS
					UE::TUniqueLock PoolInfoLock(AccessGuard);
#endif
					int32 UsedChange = (NewSize - Pool->GetBytes());
				
					// Keep as-is, reallocation isn't worth the overhead.
					BINNED_ADD_STATCOUNTER(UsedCurrent, UsedChange);
					BINNED_PEAK_STATCOUNTER(UsedPeak, UsedCurrent); //-V567
					BINNED_ADD_STATCOUNTER(WasteCurrent, (Pool->GetBytes() - NewSize));
					Pool->SetAllocationSizes(NewSize, Pool->GetOsBytes(PageSize, BinnedOSTableIndex), BinnedOSTableIndex, BinnedOSTableIndex);
				}
			}
		}
	}
	else if( Ptr == nullptr )
	{
		NewPtr = Malloc(NewSize, Alignment);
	}
	else
	{
		Free( Ptr );
		NewPtr = nullptr;
	}

	MEM_TIME(MemTime += FPlatformTime::Seconds());
	return NewPtr;
}

void FMallocBinned::Free( void* Ptr )
{
	if( !Ptr )
	{
		return;
	}

	Private::PushFreeLockless(*this, Ptr);
}

bool FMallocBinned::GetAllocationSize(void *Original, SIZE_T &SizeOut)
{
	if (!Original)
	{
		return false;
	}

	

#if PLATFORM_IOS || PLATFORM_MAC
#if USE_OS_SMALL_BLOCK_ALLOC && !USE_OS_SMALL_BLOCK_GRAB_MEMORY_FROM_OS
	if(FPlatformMemory::PtrIsOSMalloc(Original))
	{
		SizeOut = malloc_size(Original);
		return true;
	}
#endif
#endif

	UPTRINT BasePtr;
	FPoolInfo* Pool = Private::FindPoolInfo(*this, (UPTRINT)Original, BasePtr);
	PTRINT OffsetFromBase = (PTRINT)Original - (PTRINT)BasePtr;
	check(OffsetFromBase >= 0);

	if (Pool->TableIndex < BinnedOSTableIndex)
	{
		FPoolTable* Table = MemSizeToPoolTable[Pool->TableIndex];

		uint32 AlignOffset = OffsetFromBase % Table->BlockSize;

		SizeOut = Table->BlockSize - AlignOffset;
	}
	else
	{
		// if we padded out the allocation for alignment, and then offset the returned pointer from the actual allocation
		// we need to adjust for that offset. Pool->GetOsBytes() returns the entire size of the allocation, not just the 
		// usable part that was returned to the caller
		SizeOut = Pool->GetOsBytes(PageSize, BinnedOSTableIndex) - OffsetFromBase;
	}
	return true;
}

SIZE_T FMallocBinned::QuantizeSize(SIZE_T Size, uint32 Alignment)
{
	Alignment = FMath::Max<uint32>(Alignment, Private::DEFAULT_BINNED_ALLOCATOR_ALIGNMENT);
    ensure(Alignment <= PageSize);

	SIZE_T Result;
#if USE_OS_SMALL_BLOCK_ALLOC && !USE_OS_SMALL_BLOCK_GRAB_MEMORY_FROM_OS
	if (Size <= Private::SMALL_BLOCK_POOL_SIZE)
	{
		UPTRINT AlignedSize = Align(Size, Alignment);
		Result = SIZE_T(AlignedSize);
	}
	else
#endif
	if (Size < BinnedSizeLimit)
	{
		// Allocate from pool.
		FPoolTable* Table = MemSizeToPoolTable[Size];
		Result = SIZE_T(Table->BlockSize);
	}
	else if (((Size >= BinnedSizeLimit && Size <= PagePoolTable[0].BlockSize) ||
		(Size > PageSize && Size <= PagePoolTable[1].BlockSize)))
	{
		// Bucket in a pool of 3*PageSize or 6*PageSize
		uint32 BinType = Size < PageSize ? 0 : 1;
		FPoolTable* Table = &PagePoolTable[BinType];
		Result = SIZE_T(Table->BlockSize);
	}
	else
	{
		// Use OS for large allocations.
		UPTRINT AlignedSize = Align(Size, PageSize);
		Result = SIZE_T(AlignedSize);
	}
	check(Result >= Size);
	return Result;
}


bool FMallocBinned::ValidateHeap()
{
#ifdef USE_COARSE_GRAIN_LOCKS
	UE::TUniqueLock ScopedLock(AccessGuard);
#endif
	for( int32 i = 0; i < POOL_COUNT; i++ )
	{
		FPoolTable* Table = &PoolTable[i];
#ifdef USE_FINE_GRAIN_LOCKS
		UE::TUniqueLock TableLock(Table->CriticalSection);
#endif
		for( FPoolInfo** PoolPtr = &Table->FirstPool; *PoolPtr; PoolPtr = &(*PoolPtr)->Next )
		{
			FPoolInfo* Pool = *PoolPtr;
			check(Pool->PrevLink == PoolPtr);
			check(Pool->FirstMem);
			for( FFreeMem* Free = Pool->FirstMem; Free; Free = Free->Next )
			{
				check(Free->NumFreeBlocks > 0);
			}
		}
		for( FPoolInfo** PoolPtr = &Table->ExhaustedPool; *PoolPtr; PoolPtr = &(*PoolPtr)->Next )
		{
			FPoolInfo* Pool = *PoolPtr;
			check(Pool->PrevLink == PoolPtr);
			check(!Pool->FirstMem);
		}
	}

	return( true );
}

void FMallocBinned::UpdateStats()
{
	FMalloc::UpdateStats();
#if STATS
	SIZE_T	LocalOsCurrent = 0;
	SIZE_T	LocalOsPeak = 0;
	SIZE_T	LocalWasteCurrent = 0;
	SIZE_T	LocalWastePeak = 0;
	SIZE_T	LocalUsedCurrent = 0;
	SIZE_T	LocalUsedPeak = 0;
	SIZE_T	LocalCurrentAllocs = 0;
	SIZE_T	LocalTotalAllocs = 0;
	SIZE_T	LocalSlackCurrent = 0;

#if USE_OS_SMALL_BLOCK_GRAB_MEMORY_FROM_OS && ENABLE_LOW_LEVEL_MEM_TRACKER
	SIZE_T LocalNanoMallocPagesCurrent = 0;
	SIZE_T LocalNanoMallocPagesPeak = 0;
	SIZE_T LocalNanoMallocPagesWaste = 0;
	SIZE_T LocalNanoMallocWastePagesPeak = 0;
#endif //USE_OS_SMALL_BLOCK_GRAB_MEMORY_FROM_OS
	
	{
#ifdef USE_INTERNAL_LOCKS
		UE::TUniqueLock ScopedLock(AccessGuard);
#endif
		Private::UpdateSlackStat(*this);

		// Copy memory stats.
		LocalOsCurrent = OsCurrent;
		LocalOsPeak = OsPeak;
		LocalWasteCurrent = WasteCurrent;
		LocalWastePeak = WastePeak;
		LocalUsedCurrent = UsedCurrent;
		LocalUsedPeak = UsedPeak;
		LocalCurrentAllocs = CurrentAllocs;
		LocalTotalAllocs = TotalAllocs;
		LocalSlackCurrent = SlackCurrent;
		
#if USE_OS_SMALL_BLOCK_GRAB_MEMORY_FROM_OS && ENABLE_LOW_LEVEL_MEM_TRACKER
		LocalNanoMallocPagesCurrent = NanoMallocPagesCurrent;
		LocalNanoMallocPagesPeak = NanoMallocPagesPeak;
		LocalNanoMallocPagesWaste = NanoMallocPagesWaste;
		LocalNanoMallocWastePagesPeak = NanoMallocWastePagesPeak;
#endif //USE_OS_SMALL_BLOCK_GRAB_MEMORY_FROM_OS
	}

	SET_MEMORY_STAT( STAT_Binned_OsCurrent, LocalOsCurrent );
	SET_MEMORY_STAT( STAT_Binned_OsPeak, LocalOsPeak );
	SET_MEMORY_STAT( STAT_Binned_WasteCurrent, LocalWasteCurrent );
	SET_MEMORY_STAT( STAT_Binned_WastePeak, LocalWastePeak );
	SET_MEMORY_STAT( STAT_Binned_UsedCurrent, LocalUsedCurrent );
	SET_MEMORY_STAT( STAT_Binned_UsedPeak, LocalUsedPeak );
	SET_DWORD_STAT( STAT_Binned_CurrentAllocs, LocalCurrentAllocs );
	SET_DWORD_STAT( STAT_Binned_TotalAllocs, LocalTotalAllocs );
	SET_MEMORY_STAT( STAT_Binned_SlackCurrent, LocalSlackCurrent );
	
#if USE_OS_SMALL_BLOCK_GRAB_MEMORY_FROM_OS && ENABLE_LOW_LEVEL_MEM_TRACKER
	SET_MEMORY_STAT( STAT_Binned_NanoMallocPages_Current, LocalNanoMallocPagesCurrent );
	SET_MEMORY_STAT( STAT_Binned_NanoMallocPages_Peak, LocalNanoMallocPagesPeak );
	SET_MEMORY_STAT( STAT_Binned_NanoMallocPages_WasteCurrent, LocalNanoMallocPagesWaste );
	SET_MEMORY_STAT( STAT_Binned_NanoMallocPages_WastePeak, LocalNanoMallocWastePagesPeak );
#endif //USE_OS_SMALL_BLOCK_GRAB_MEMORY_FROM_OS
#endif
}

void FMallocBinned::DumpAllocatorStats( class FOutputDevice& Ar )
{
	FBufferedOutputDevice BufferedOutput;
	{
#ifdef USE_COARSE_GRAIN_LOCKS
		UE::TUniqueLock ScopedLock(AccessGuard);
#endif
		ValidateHeap();
#if STATS
		Private::UpdateSlackStat(*this);
#if !NO_LOGGING
		// This is all of the memory including stuff too big for the pools
		BufferedOutput.CategorizedLogf( LogMemory.GetCategoryName(), ELogVerbosity::Log, TEXT( "Allocator Stats for %s:" ), GetDescriptiveName() );
		// Waste is the total overhead of the memory system
		BufferedOutput.CategorizedLogf( LogMemory.GetCategoryName(), ELogVerbosity::Log, TEXT( "Current Memory %.2f MB used, plus %.2f MB waste" ), UsedCurrent / (1024.0f * 1024.0f), (OsCurrent - UsedCurrent) / (1024.0f * 1024.0f) );  //-V567
		BufferedOutput.CategorizedLogf( LogMemory.GetCategoryName(), ELogVerbosity::Log, TEXT( "Peak Memory %.2f MB used, plus %.2f MB waste" ), UsedPeak / (1024.0f * 1024.0f), (OsPeak - UsedPeak) / (1024.0f * 1024.0f) ); //-V567

		BufferedOutput.CategorizedLogf( LogMemory.GetCategoryName(), ELogVerbosity::Log, TEXT( "Current OS Memory %.2f MB, peak %.2f MB" ), OsCurrent / (1024.0f * 1024.0f), OsPeak / (1024.0f * 1024.0f) ); //-V567
		BufferedOutput.CategorizedLogf( LogMemory.GetCategoryName(), ELogVerbosity::Log, TEXT( "Current Waste %.2f MB, peak %.2f MB" ), WasteCurrent / (1024.0f * 1024.0f), WastePeak / (1024.0f * 1024.0f) ); //-V567
		BufferedOutput.CategorizedLogf( LogMemory.GetCategoryName(), ELogVerbosity::Log, TEXT( "Current Used %.2f MB, peak %.2f MB" ), UsedCurrent / (1024.0f * 1024.0f), UsedPeak / (1024.0f * 1024.0f) ); //-V567
		BufferedOutput.CategorizedLogf( LogMemory.GetCategoryName(), ELogVerbosity::Log, TEXT( "Current Slack %.2f MB" ), SlackCurrent / (1024.0f * 1024.0f) );

		BufferedOutput.CategorizedLogf( LogMemory.GetCategoryName(), ELogVerbosity::Log, TEXT( "Allocs      % 6i Current / % 6i Total" ), CurrentAllocs, TotalAllocs ); //-V567
		MEM_TIME( BufferedOutput.CategorizedLogf( LogMemory.GetCategoryName(), ELogVerbosity::Log, TEXT( "Seconds     % 5.3f" ), MemTime ) );
		MEM_TIME( BufferedOutput.CategorizedLogf( LogMemory.GetCategoryName(), ELogVerbosity::Log, TEXT( "MSec/Allc   % 5.5f" ), 1000.0 * MemTime / MemAllocs ) );

#if USE_OS_SMALL_BLOCK_GRAB_MEMORY_FROM_OS
		BufferedOutput.CategorizedLogf( LogMemory.GetCategoryName(), ELogVerbosity::Log, TEXT( "NanoMallocPage Stats:" ) );
		BufferedOutput.CategorizedLogf( LogMemory.GetCategoryName(), ELogVerbosity::Log, TEXT( "Current %.2f MB, peak %.2f MB" ), NanoMallocPagesCurrent / (1024.0f * 1024.0f),  NanoMallocPagesPeak / (1024.0f * 1024.0f) );
		BufferedOutput.CategorizedLogf( LogMemory.GetCategoryName(), ELogVerbosity::Log, TEXT( "Current Waste %.2f MB, peak %.2f MB" ), NanoMallocPagesWaste / (1024.0f * 1024.0f),  NanoMallocWastePagesPeak / (1024.0f * 1024.0f) );
#endif //USE_OS_SMALL_BLOCK_GRAB_MEMORY_FROM_OS
		
		// This is the memory tracked inside individual allocation pools
		BufferedOutput.CategorizedLogf( LogMemory.GetCategoryName(), ELogVerbosity::Log, TEXT( "" ) );
		BufferedOutput.CategorizedLogf( LogMemory.GetCategoryName(), ELogVerbosity::Log, TEXT( "Block Size Num Pools Max Pools Cur Allocs Total Allocs Min Req Max Req Mem Used Mem Slack Mem Waste Efficiency" ) );
		BufferedOutput.CategorizedLogf( LogMemory.GetCategoryName(), ELogVerbosity::Log, TEXT( "---------- --------- --------- ---------- ------------ ------- ------- -------- --------- --------- ----------" ) );

		uint32 TotalMemory = 0;
		uint32 TotalWaste = 0;
		uint32 TotalActiveRequests = 0;
		uint32 TotalTotalRequests = 0;
		uint32 TotalPools = 0;
		uint32 TotalSlack = 0;

		FPoolTable* Table = nullptr;
		for( int32 i = 0; i < BinnedSizeLimit + EXTENDED_PAGE_POOL_ALLOCATION_COUNT; i++ )
		{
			if( Table == MemSizeToPoolTable[i] || MemSizeToPoolTable[i]->BlockSize == 0 )
				continue;

			Table = MemSizeToPoolTable[i];

#ifdef USE_FINE_GRAIN_LOCKS
			Table->CriticalSection.Lock();
#endif

			uint32 TableAllocSize = (Table->BlockSize > BinnedSizeLimit ? (((3 * (i - BinnedSizeLimit)) + 3)*Private::BINNED_ALLOC_POOL_SIZE) : Private::BINNED_ALLOC_POOL_SIZE);
			// The amount of memory allocated from the OS
			uint32 MemAllocated = (Table->NumActivePools * TableAllocSize) / 1024;
			// Amount of memory actually in use by allocations
			uint32 MemUsed = (Table->BlockSize * Table->ActiveRequests) / 1024;
			// Wasted memory due to pool size alignment
			uint32 PoolMemWaste = Table->NumActivePools * (TableAllocSize - ((TableAllocSize / Table->BlockSize) * Table->BlockSize)) / 1024;
			// Wasted memory due to individual allocation alignment. This is an estimate.
			uint32 MemWaste = (uint32)(((double)Table->TotalWaste / (double)Table->TotalRequests) * (double)Table->ActiveRequests) / 1024 + PoolMemWaste;
			// Memory that is reserved in active pools and ready for future use
			uint32 MemSlack = MemAllocated - MemUsed - PoolMemWaste;
			// Copy the other stats before releasing the lock and calling CategorizedLogf
			uint32 TableBlockSize = Table->BlockSize;
			uint32 TableNumActivePools = Table->NumActivePools;
			uint32 TableMaxActivePools = Table->MaxActivePools;
			uint32 TableActiveRequests = Table->ActiveRequests;
			uint32 TableTotalRequests = (uint32)Table->TotalRequests;
			uint32 TableMinRequest = Table->MinRequest;
			uint32 TableMaxRequest = Table->MaxRequest;

#ifdef USE_FINE_GRAIN_LOCKS
			Table->CriticalSection.Unlock();
#endif

			BufferedOutput.CategorizedLogf( LogMemory.GetCategoryName(), ELogVerbosity::Log, TEXT( "% 10i % 9i % 9i % 10i % 12i % 7i % 7i % 7iK % 8iK % 8iK % 9.2f%%" ),
										  TableBlockSize,
										  TableNumActivePools,
										  TableMaxActivePools,
										  TableActiveRequests,
										  TableTotalRequests,
										  TableMinRequest,
										  TableMaxRequest,
										  MemUsed,
										  MemSlack,
										  MemWaste,
										  MemAllocated ? 100.0f * (MemAllocated - MemWaste) / MemAllocated : 100.0f );

			TotalMemory += MemAllocated;
			TotalWaste += MemWaste;
			TotalSlack += MemSlack;
			TotalActiveRequests += TableActiveRequests;
			TotalTotalRequests += TableTotalRequests;
			TotalPools += TableNumActivePools;
		}

		BufferedOutput.CategorizedLogf( LogMemory.GetCategoryName(), ELogVerbosity::Log, TEXT( "" ) );
		BufferedOutput.CategorizedLogf( LogMemory.GetCategoryName(), ELogVerbosity::Log, TEXT( "%iK allocated in pools (with %iK slack and %iK waste). Efficiency %.2f%%" ), TotalMemory, TotalSlack, TotalWaste, TotalMemory ? 100.0f * (TotalMemory - TotalWaste) / TotalMemory : 100.0f );
		BufferedOutput.CategorizedLogf( LogMemory.GetCategoryName(), ELogVerbosity::Log, TEXT( "Allocations %i Current / %i Total (in %i pools)" ), TotalActiveRequests, TotalTotalRequests, TotalPools );
		BufferedOutput.CategorizedLogf( LogMemory.GetCategoryName(), ELogVerbosity::Log, TEXT( "" ) );
#endif
#endif
	}

	BufferedOutput.RedirectTo( Ar );
}

const TCHAR* FMallocBinned::GetDescriptiveName()
{
	return TEXT("binned");
}

PRAGMA_RESTORE_UNSAFE_TYPECAST_WARNINGS
