// Copyright Epic Games, Inc. All Rights Reserved.

#include "HAL/MallocBinned2.h"
#include "Logging/LogVerbosity.h"
#include "Misc/AssertionMacros.h"
#include "Templates/Function.h"
#include "GenericPlatform/GenericPlatformProcess.h"
#include "Stats/Stats.h"
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformMisc.h"
#include "Misc/App.h"
#include "Misc/CoreDelegates.h"
#include "HAL/MallocTimer.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "HAL/MallocBinnedCommonUtils.h"

CSV_DECLARE_CATEGORY_MODULE_EXTERN(CORE_API, FMemory);

PRAGMA_DISABLE_UNSAFE_TYPECAST_WARNINGS


#define UE_DEFAULT_GBinned2MoveOSFreesOffTimeCriticalThreads 1

#if UE_MBC_ALLOW_RUNTIME_TWEAKING
	int32 GBinned2MoveOSFreesOffTimeCriticalThreads = UE_DEFAULT_GBinned2MoveOSFreesOffTimeCriticalThreads;
	static FAutoConsoleVariableRef GGMallocBinned2MoveOSFreesOffTimeCriticalThreadsCVar(
		TEXT("MallocBinned2.MoveOSFreesOffTimeCriticalThreads"),
		GBinned2MoveOSFreesOffTimeCriticalThreads,
		TEXT("When the OS needs to free memory hint to the underlying cache that we are on a time critical thread, it may decide to delay the free for a non time critical thread")
	);
#else
#	define GBinned2MoveOSFreesOffTimeCriticalThreads	UE_DEFAULT_GBinned2MoveOSFreesOffTimeCriticalThreads
#endif

#if UE_MB2_DOUBLE_FREE_DETECTION
	bool GMallocBinned2EnableSmallDoubleFreeDetection = true;
	static FAutoConsoleVariableRef GGMallocBinned2EnableSmallDoubleFreeDetectionCVar(
		TEXT("MallocBinned2.EnableSmallDoubleFreeDetection"),
		GMallocBinned2EnableSmallDoubleFreeDetection,
		TEXT("Enables the double-free detector for small allocations in FMallocBinned2"));
#endif 

#if UE_MB2_ALLOCATOR_STATS
	int64 Binned2HashMemory = 0;
#endif

// Bin sizes are based around getting the maximum amount of allocations per pool, with as little alignment waste as possible.
// Bin sizes should be close to even divisors of the system page size, and well distributed.
// They must be 16-byte aligned as well.
constexpr uint16 SmallBinSizesInternal[] =
{
	16, 32, 48, 64, 80, 96, 112, 128, 144, 160, 176, 192, 208, // +16
//	Bin		|	Divider | Slack leftover per page (64KB)
	256,		// /256
	288,		// /227 160b
	320,		// /204 256b
	384,		// /170 256b
	448,		// /146 128b
	512,		// /128	
	560,		// /117	16b
	624,		// /105	16b
	720,		// /91	16b
	816,		// /80	256b
	912,		// /71	784b
	1024-16,	// /64
	1168,		// /56	128b
	1392,		// /47	112b
	1520,		// /43	176b
	1680,		// /39	16b
	1872,		// /35	16b
	2048-16,	// /32
	2256,		// /29	112b
	2608,		// /25	336b
	2976,		// /22	64b
	3264,		// /20	256b
	3632,		// /18	160b
	4096-16,	// /16
	4368,		// /15	16b
	4672,		// /14	128b
	5040,		// /13	16b
	5456,		// /12	64b
	5952,		// /11	64b
	6544,		// /10	96b
	7280,		// /9	16b
	8192-16,	// /8
	9360,		// /7	16b
	10912,		// /6	64b
	13104,		// /5	16b
#if !AGGRESSIVE_MEMORY_SAVING
	16384-16,	// /4
	21840,		// /3	16b
	32768-16	// /2
#endif
};

static uint32 GMB2PageSize = 0;

uint16 FMallocBinned2::SmallBinSizes[UE_MB2_SMALL_POOL_COUNT] = { 0 };
FMallocBinned2* FMallocBinned2::MallocBinned2 = nullptr;
// Mapping of sizes to small table indices
uint8 FMallocBinned2::MemSizeToPoolIndex[SIZE_TO_POOL_INDEX_NUM] = { 0 };

template <>
thread_local FMallocBinned2::FPerThreadFreeBlockLists* FMallocBinned2::FPerThreadFreeBlockLists::ThreadSingleton = nullptr;


static FORCEINLINE bool IsSupportedSize(SIZE_T Size)
{
	bool bResult = IsAligned(Size, UE_MBC_MIN_SMALL_POOL_ALIGNMENT);
	bResult = bResult && (Size >> UE_MBC_BIN_SIZE_SHIFT) <= SIZE_T(MAX_uint32);
	return bResult;
}

#if UE_MB2_DOUBLE_FREE_DETECTION
// Canary stored after the FFreeBlock header area in freed bins to detect double frees.
// Placed at sizeof(FFreeBlock) so it survives both FBundleNode (TLS) and FFreeBlock (pool) overwrites.
// XOR with pointer address makes false positives from user data ~2^-64 probability.
FORCEINLINE void FMallocBinned2::SetAndValidateDoubleFreeCanary(void* Ptr)
{
	if (GMallocBinned2EnableSmallDoubleFreeDetection)
	{
		static constexpr uintptr_t DoubleFreeCanaryMagic = 0xFE2DF3EEDEADF3EE;
		const uintptr_t FreedMarker = (uintptr_t)Ptr ^ DoubleFreeCanaryMagic;
		uintptr_t* CanarySlot = (uintptr_t*)((uint8*)Ptr + sizeof(FFreeBlock));
		if (UNLIKELY(*CanarySlot == FreedMarker))
		{
			ReportSmallDoubleFree(Ptr);
		}
		*CanarySlot = FreedMarker;
	}
}

FORCEINLINE void FMallocBinned2::ClearDoubleFreeCanary(void* Ptr)
{
	if (GMallocBinned2EnableSmallDoubleFreeDetection)
	{
		uintptr_t* CanarySlot = (uintptr_t*)((uint8*)Ptr + sizeof(FFreeBlock));
		*CanarySlot = 0;
	}
}

FORCENOINLINE void FMallocBinned2::ReportSmallDoubleFree(void* Ptr)
{
#if DO_ENSURE
	ensureAlwaysMsgf(false, TEXT("FMallocBinned2 Suspected double-free when freeing pointer: %p"), Ptr);
#else
	UE_LOGF(LogMemory, Warning, "FMallocBinned2 Suspected double-free when freeing pointer: %p", Ptr);
	FDebug::DumpStackTraceToLog(ELogVerbosity::Warning);
#endif
}

#endif

FMallocBinned2::FPoolInfo::FPoolInfo()
	: Taken(0)
	, Canary(ECanary::Unassigned)
	, AllocSize(0)
	, FirstFreeBlock(nullptr)
	, Next(nullptr)
	, PtrToPrevNext(nullptr)
{
}

void FMallocBinned2::FPoolInfo::CheckCanary(ECanary ShouldBe) const
{
	if (Canary != ShouldBe)
	{
		UE_LOGF(LogMemory, Fatal, "MallocBinned2 Corruption Canary was 0x%x, should be 0x%x", int32(Canary), int32(ShouldBe));
	}
}

void FMallocBinned2::FPoolInfo::SetCanary(ECanary ShouldBe, bool bPreexisting, bool bGuaranteedToBeNew)
{
	if (bPreexisting)
	{
		if (bGuaranteedToBeNew)
		{
			UE_LOGF(LogMemory, Fatal, "MallocBinned2 Corruption Canary was 0x%x, should be 0x%x. This block is both preexisting and guaranteed to be new; which makes no sense.", int32(Canary), int32(ShouldBe));
		}
		if (ShouldBe == ECanary::Unassigned)
		{
			if (Canary != ECanary::FirstFreeBlockIsOSAllocSize && Canary != ECanary::FirstFreeBlockIsPtr)
			{
				UE_LOGF(LogMemory, Fatal, "MallocBinned2 Corruption Canary was 0x%x, will be 0x%x because this block should be preexisting and in use.", int32(Canary), int32(ShouldBe));
			}
		}
		else if (Canary != ShouldBe)
		{
			UE_LOGF(LogMemory, Fatal, "MallocBinned2 Corruption Canary was 0x%x, should be 0x%x because this block should be preexisting.", int32(Canary), int32(ShouldBe));
		}
	}
	else
	{
		if (bGuaranteedToBeNew)
		{
			if (Canary != ECanary::Unassigned)
			{
				UE_LOGF(LogMemory, Fatal, "MallocBinned2 Corruption Canary was 0x%x, will be 0x%x. This block is guaranteed to be new yet is it already assigned.", int32(Canary), int32(ShouldBe));
			}
		}
		else if (Canary != ShouldBe && Canary != ECanary::Unassigned)
		{
			UE_LOGF(LogMemory, Fatal, "MallocBinned2 Corruption Canary was 0x%x, will be 0x%x does not have an expected value.", int32(Canary), int32(ShouldBe));
		}
	}
	Canary = ShouldBe;
}

bool FMallocBinned2::FPoolInfo::HasFreeBin() const
{
	CheckCanary(ECanary::FirstFreeBlockIsPtr);
	return FirstFreeBlock && FirstFreeBlock->GetNumFreeBins() != 0;
}

void* FMallocBinned2::FPoolInfo::AllocateBin(uint32 BinSize)
{
	check(HasFreeBin());
	++Taken;
	void* Result = FirstFreeBlock->AllocateBin(BinSize);
	ExhaustPoolIfNecessary();
	return Result;
}

SIZE_T FMallocBinned2::FPoolInfo::GetOSRequestedBytes() const
{
	return SIZE_T(AllocSize) << UE_MBC_BIN_SIZE_SHIFT;
}

SIZE_T FMallocBinned2::FPoolInfo::GetOsAllocatedBytes() const
{
	CheckCanary(ECanary::FirstFreeBlockIsOSAllocSize);
	return (SIZE_T)FirstFreeBlock;
}

void FMallocBinned2::FPoolInfo::SetOSAllocationSizes(SIZE_T InRequestedBytes, UPTRINT InAllocatedBytes)
{
	CheckCanary(ECanary::FirstFreeBlockIsOSAllocSize);
	checkSlow(InRequestedBytes != 0);                // Shouldn't be pooling zero byte allocations
	checkSlow(InAllocatedBytes >= InRequestedBytes); // We must be allocating at least as much as we requested
	checkSlow(IsSupportedSize(InRequestedBytes));    // We must be allocating a size we can store

	AllocSize      = uint32(InRequestedBytes >> UE_MBC_BIN_SIZE_SHIFT);
	FirstFreeBlock = (FFreeBlock*)InAllocatedBytes;
}

void FMallocBinned2::FPoolInfo::Link(FPoolInfo*& PrevNext)
{
	if (PrevNext)
	{
		PrevNext->PtrToPrevNext = &Next;
	}
	Next          = PrevNext;
	PtrToPrevNext = &PrevNext;
	PrevNext      = this;
}

void FMallocBinned2::FPoolInfo::Unlink()
{
	if (Next)
	{
		Next->PtrToPrevNext = PtrToPrevNext;
	}
	*PtrToPrevNext = Next;
}

void FMallocBinned2::FPoolInfo::ExhaustPoolIfNecessary()
{
	if (FirstFreeBlock->GetNumFreeBins() == 0)
	{
		FirstFreeBlock = FirstFreeBlock->GetNextFreeBlock();
	}
	check(!FirstFreeBlock || FirstFreeBlock->GetNumFreeBins() != 0);
}

struct FMallocBinned2::Private
{
	static MallocBinnedPrivate::TGlobalRecycler<UE_MB2_SMALL_POOL_COUNT> GGlobalRecycler;

	static void FreeBundles(FMallocBinned2& Allocator, FBundleNode* BundlesToRecycle, uint32 InPoolIndex)
	{
		FPoolTable& Table = Allocator.SmallPoolTables[InPoolIndex];
		const uint32 BinSize = Table.BinSize;

		UE::TUniqueLock Lock(Table.Mutex);

		FBundleNode* Node = BundlesToRecycle;
		do
		{
			FBundleNode* NextNode = Node->GetNextNodeInCurrentBundle();
			FPoolInfo*   NodePool = Internal::FindPoolInfo(Allocator, Node);
			if (!NodePool)
			{
				UE_LOGF(LogMemory, Fatal, "FMallocBinned2 Attempt to free an unrecognized small block %p", Node);
			}
			NodePool->CheckCanary(FPoolInfo::ECanary::FirstFreeBlockIsPtr);

			// If this pool was exhausted, move to available list.
			if (!NodePool->FirstFreeBlock)
			{
				Table.ActivePools.LinkToFront(NodePool);
			}
			else
			{
				// If we are freeing memory in this pool it must have the current canary and not the pre-fork one. All caches should have been cleared when forking.
				check(GetPoolHeaderFromPointer(NodePool->FirstFreeBlock)->CanaryAndForkState == EBlockCanary::Zero || GetPoolHeaderFromPointer(NodePool->FirstFreeBlock)->CanaryAndForkState == Allocator.CurrentCanary);
			}

			// Free a pooled allocation.
			FFreeBlock* Free = (FFreeBlock*)Node;
			Free->NumFreeBins = 1;
			Free->SetNextFreeBlock(NodePool->FirstFreeBlock);
			NodePool->FirstFreeBlock = Free;

			UE_MBC_UPDATE_STATS(--Table.TotalUsedBins);

			// Free this pool.
			check(NodePool->Taken >= 1);
			if (--NodePool->Taken == 0)
			{
				NodePool->SetCanary(FPoolInfo::ECanary::Unassigned, true, false);
				FPoolHeader* BasePtrOfNode = GetPoolHeaderFromPointer(Node);

				// Free the OS memory.
				NodePool->Unlink();
				{
					UE::TUniqueLock InnerLock(Allocator.ExternalAllocMutex);
					Allocator.CachedOSPageAllocator.Free(BasePtrOfNode, GMB2PageSize);
				}

#if UE_MB2_ALLOCATOR_STATS
				int NumBinsInBlock = GMB2PageSize / BinSize;
				if (NumBinsInBlock * BinSize + sizeof(FPoolHeader) > GMB2PageSize)
				{
					NumBinsInBlock--;
				}

				Table.TotalAllocatedBins -= NumBinsInBlock;
				Table.TotalAllocatedMem -= GMB2PageSize;
				AllocatedOSSmallPoolMemory.fetch_sub((int64)GMB2PageSize, std::memory_order_relaxed);
#endif
			}

			Node = NextNode;
		} while (Node);
	}

#if BINNED2_FORK_SUPPORT
	static void CheckThreadFreeBlockListsForFork()
	{
		if(GMallocBinnedPerThreadCaches)
		{
			UE::TUniqueLock Lock(GetFreeBlockListsRegistrationMutex());
			TArray<FPerThreadFreeBlockLists*>& List = GetRegisteredFreeBlockLists();
			UE_CLOGF(List.Num() == 1 && List[0] != FPerThreadFreeBlockLists::Get(), LogMemory, Fatal, "There was a thread-local free list at fork time which did not belong to the main forking thread. No other threads should be alive at fork time. If threads are spawned before forking, they must be killed and FMallocBinned2::ClearAndDisableTLSCachesOnCurrentThread() must be called.");
			UE_CLOGF(List.Num() > 1, LogMemory, Fatal, "There were multiple thread-local free lists at fork time. No other threads should be alive at fork time. If threads are spawned before forking, they must be killed and FMallocBinned2::ClearAndDisableTLSCachesOnCurrentThread() must be called.");
		}
	}
#endif
};

MallocBinnedPrivate::TGlobalRecycler<UE_MB2_SMALL_POOL_COUNT> FMallocBinned2::Private::GGlobalRecycler;

FORCEINLINE void FMallocBinned2::FPoolList::Clear()
{
	Front = nullptr;
}

FORCEINLINE bool FMallocBinned2::FPoolList::IsEmpty() const
{
	return Front == nullptr;
}

FORCEINLINE FMallocBinned2::FPoolInfo& FMallocBinned2::FPoolList::GetFrontPool()
{
	check(!IsEmpty());
	return *Front;
}

FORCEINLINE const FMallocBinned2::FPoolInfo& FMallocBinned2::FPoolList::GetFrontPool() const
{
	check(!IsEmpty());
	return *Front;
}

void FMallocBinned2::FPoolList::LinkToFront(FPoolInfo* Pool)
{
	Pool->Unlink();
	Pool->Link(Front);
}

FMallocBinned2::FPoolInfo& FMallocBinned2::FPoolList::PushNewPoolToFront(FMallocBinned2& Allocator, FPoolTable& Table, uint32 InPoolIndex)
{
	LLM_PLATFORM_SCOPE(ELLMTag::FMalloc);

	void* FreePtr;
	{
		UE::TUniqueLock Lock(Allocator.ExternalAllocMutex);
		FreePtr = Allocator.CachedOSPageAllocator.Allocate(GMB2PageSize, FMemory::AllocationHints::SmallPool);
	}
	if (!FreePtr)
	{
		OutOfMemory(GMB2PageSize);
	}

	FPoolHeader* Header = new (FreePtr) FPoolHeader(GMB2PageSize, Table.BinSize, InPoolIndex, Allocator.CurrentCanary);
	check(IsAligned(Header, GMB2PageSize));

#if UE_MB2_ALLOCATOR_STATS
	Table.TotalAllocatedBins += Header->NumFreeBins;
	Table.TotalAllocatedMem += GMB2PageSize;
	AllocatedOSSmallPoolMemory.fetch_add((int64)GMB2PageSize, std::memory_order_relaxed);
#endif

	UE::TUniqueLock Lock(Allocator.ExternalAllocMutex);
	// Create pool
	FPoolInfo* Result = Internal::GetOrCreatePoolInfo(Allocator, Header, FPoolInfo::ECanary::FirstFreeBlockIsPtr);
	Result->Link(Front);
	Result->Taken          = 0;
	Result->FirstFreeBlock = Header;

	return *Result;
}

FMallocBinned2::FMallocBinned2()
{
	static bool bOnce = false;
	check(!bOnce); // this is now a singleton-like thing and you cannot make multiple copies
	bOnce = true;

	FMemory::Memcpy(SmallBinSizes, SmallBinSizesInternal);
	
	static_assert(sizeof(FFreeBlock)  <= SmallBinSizesInternal[0], "sizeof(FFreeBlock)  must be fit in smallest allocation size handled by FMallocBinned2.");
	static_assert(sizeof(FBundleNode) <= SmallBinSizesInternal[0], "sizeof(FBundleNode) must fit in smallest allocation size handled by FMallocBinned2.");
	static_assert(sizeof(FFreeBlock) == 8, "FFreeBlock should fit into 8 bytes");
#if UE_MB2_DOUBLE_FREE_DETECTION
	static_assert(sizeof(FBundleNode) <= sizeof(FFreeBlock), "FBundleNode must fit within FFreeBlock.");
#endif

	const FGenericPlatformMemoryConstants Constants = FPlatformMemory::GetConstants();
	GMB2PageSize = Constants.BinnedPageSize;
	OsAllocationGranularity = Constants.BinnedAllocationGranularity ? Constants.BinnedAllocationGranularity : GMB2PageSize;
	NumPoolsPerPage = GMB2PageSize / sizeof(FPoolInfo);
	PtrToPoolMapping.Init(GMB2PageSize, NumPoolsPerPage, Constants.AddressStart, Constants.AddressLimit);

	checkf(FMath::IsPowerOfTwo(GMB2PageSize), TEXT("OS page size must be a power of two"));
	checkf(Constants.AddressLimit > GMB2PageSize, TEXT("OS address limit must be greater than the page size")); // Check to catch 32 bit overflow in AddressLimit
	checkf(GMB2PageSize % UE_MB2_LARGE_ALLOC == 0, TEXT("OS page size must be a multiple of UE_MB2_LARGE_ALLOC"));

	static_assert(SmallBinSizesInternal[UE_MB2_SMALL_POOL_COUNT - 1] == UE_MB2_MAX_SMALL_POOL_SIZE, "UE_MB2_MAX_SMALL_POOL_SIZE must equal the largest bin size");
	static_assert(UE_ARRAY_COUNT(SmallBinSizesInternal) == UE_MB2_SMALL_POOL_COUNT, "Number of bins in SmallBinSizesInternal must match UE_MB2_SMALL_POOL_COUNT");
	static_assert(UE_ARRAY_COUNT(SmallBinSizesInternal) <= 256, "Number of bins in SmallBinSizesInternal fit in one byte");

	// Init pool tables.
	for (uint32 Index = 0; Index != UE_MB2_SMALL_POOL_COUNT; ++Index)
	{
		checkf(Index == 0 || SmallBinSizes[Index - 1] < SmallBinSizes[Index], TEXT("Small bin sizes must be strictly increasing"));
		checkf(SmallBinSizes[Index] <= GMB2PageSize, TEXT("Small bin size must be small enough to fit into a page"));
		checkf(SmallBinSizes[Index] % UE_MBC_MIN_SMALL_POOL_ALIGNMENT == 0, TEXT("Small bin size must be a multiple of UE_MBC_MINIMUM_ALIGNMENT"));

		SmallPoolTables[Index].BinSize = SmallBinSizes[Index];
	}

	// Set up pool mappings
	uint8* IndexEntry = MemSizeToPoolIndex;
	uint32 PoolIndex  = 0;
	for (uint32 Index = 0; Index != SIZE_TO_POOL_INDEX_NUM; ++Index)
	{
		const uint32 BinSize = Index << UE_MBC_BIN_SIZE_SHIFT; // inverse of int32 Index = int32((Size >> UE_MBC_BIN_SIZE_SHIFT));
		while (SmallBinSizes[PoolIndex] < BinSize)
		{
			++PoolIndex;
			check(PoolIndex != UE_MB2_SMALL_POOL_COUNT);
		}
		check(PoolIndex < 256);
		*IndexEntry++ = uint8(PoolIndex);
	}

	AllocateHashBuckets();

	MallocBinned2 = this;
	GFixedMallocLocationPtr = (FMalloc**)(&MallocBinned2);
}

FMallocBinned2::~FMallocBinned2()
{
}

void FMallocBinned2::OnMallocInitialized()
{
	TMallocBinnedCommon::OnMallocInitialized();

#if UE_USE_VERYLARGEPAGEALLOCATOR
	FCoreDelegates::GetLowLevelAllocatorMemoryTrimDelegate().AddLambda([this]()
		{
			UE::TUniqueLock Lock(ExternalAllocMutex);
			CachedOSPageAllocator.FreeAll(&ExternalAllocMutex);
		}
	);

	FCoreDelegates::GetRefreshLowLevelAllocatorDelegate().AddLambda([this]()
		{
			UE::TUniqueLock Lock(ExternalAllocMutex);
			CachedOSPageAllocator.Refresh();
		}
	);
#endif
}

void FMallocBinned2::OnPreFork()
{
#if BINNED2_FORK_SUPPORT
	// Trim caches so we don't use them in the child process and cause pages to be copied
	if (GMallocBinnedPerThreadCaches)
	{
		FMallocBinnedCommonUtils::FlushCurrentThreadCache(*this);
		FMallocBinned2::Private::CheckThreadFreeBlockListsForFork();
	}

	for (int32 PoolIndex = 0; PoolIndex < UE_ARRAY_COUNT(SmallPoolTables); ++PoolIndex)
	{
		while (FBundleNode* Node = FMallocBinned2::Private::GGlobalRecycler.PopBundle(PoolIndex))
		{
			FMallocBinned2::Private::FreeBundles(*this, Node, PoolIndex);
		}
	}

#if !UE_USE_VERYLARGEPAGEALLOCATOR
	UE::TUniqueLock Lock(ExternalAllocMutex);
	CachedOSPageAllocator.FreeAll(&ExternalAllocMutex);
#endif

#endif // BINNED2_FORK_SUPPORT
}

void FMallocBinned2::OnPostFork()
{
#if BINNED2_FORK_SUPPORT
	if (GMallocBinnedPerThreadCaches)
	{
		FMallocBinnedCommonUtils::FlushCurrentThreadCache(*this);
		FMallocBinned2::Private::CheckThreadFreeBlockListsForFork();
	}

	// This will be compared against the pool header of existing allocations to turn Free into a no-op for pages shared with the parent process
	UE_CLOGF(CurrentCanary != EBlockCanary::PreFork, LogMemory, Fatal, "FMallocBinned2 only supports forking once!");

	OldCanary = CurrentCanary;
	CurrentCanary = EBlockCanary::PostFork;

	for (FPoolTable& Table : SmallPoolTables)
	{
		UE::TUniqueLock Lock(Table.Mutex);
		// Clear our list of partially used pages so we don't dirty them and cause them to become unshared with the parent process
		Table.ActivePools.Clear();
		Table.ExhaustedPools.Clear();
	}
#endif
}

bool FMallocBinned2::IsInternallyThreadSafe() const
{ 
	return true;
}

FORCEINLINE static bool UseSmallAlloc(SIZE_T Size, uint32 Alignment)
{
	return ((Size <= UE_MB2_MAX_SMALL_POOL_SIZE) & (Alignment <= UE_MBC_STANDARD_ALIGNMENT)); // one branch, not two
}

void* FMallocBinned2::Malloc(SIZE_T Size, uint32 Alignment)
{
	// Only allocate from the small pools if the size is small enough and the alignment isn't crazy large.
	// With large alignments, we'll waste a lot of memory allocating an entire page, but such alignments are highly unlikely in practice.
	bool bUseSmallPool = UseSmallAlloc(Size, Alignment);
	if (!bUseSmallPool)
	{
		bUseSmallPool = PromoteToLargerBin(Size, Alignment, *this);
	}

	if (bUseSmallPool)
	{
		return MallocExternalSmall(Size, Alignment);
	}
	return MallocExternalLarge(Size, Alignment);
}

void* FMallocBinned2::MallocExternalSmall(SIZE_T Size, uint32 Alignment)
{
	const uint32 PoolIndex = BoundSizeToPoolIndex(Size, MemSizeToPoolIndex);
	FPerThreadFreeBlockLists* Lists = GMallocBinnedPerThreadCaches ? FPerThreadFreeBlockLists::Get() : nullptr;
	if (Lists)
	{
		if (Lists->ObtainRecycledPartial(PoolIndex, Private::GGlobalRecycler))
		{
			if (void* Result = Lists->Malloc(PoolIndex))
			{
#if UE_MB2_DOUBLE_FREE_DETECTION
				ClearDoubleFreeCanary(Result);
#endif
#if UE_MB2_ALLOCATOR_STATS
				const uint32 BinSize = PoolIndexToBinSize(PoolIndex);
				Lists->AllocatedMemory += BinSize;
#endif
				return Result;
			}
		}
	}

	NOALLOC_SCOPE_CYCLE_COUNTER(STAT_FMallocBinned2_MallocExternalSmall);

	// Allocate from small object pool.
	FPoolTable& Table = SmallPoolTables[PoolIndex];

	UE::TUniqueLock Lock(Table.Mutex);

	FPoolInfo* Pool;
	if (!Table.ActivePools.IsEmpty())
	{
		Pool = &Table.ActivePools.GetFrontPool();
	}
	else
	{
		Pool = &Table.ActivePools.PushNewPoolToFront(*this, Table, PoolIndex);
	}

	void* Result = Pool->AllocateBin(Table.BinSize);
#if UE_MB2_DOUBLE_FREE_DETECTION
	ClearDoubleFreeCanary(Result);
#endif
#if UE_MB2_ALLOCATOR_STATS
	++Table.TotalUsedBins;
	AllocatedSmallPoolMemory.fetch_add(Table.BinSize, std::memory_order_relaxed);
#endif

#if UE_MBC_LIGHTWEIGHT_BIN_CALLSTACK_TRACKER
	TrackBinAllocation(Table.BinSize, Table.TotalAllocatedMem);
#endif

	if (GMallocBinnedAllocExtra)
	{
		if (Lists)
		{
			// prefill the free list with some allocations so we are less likely to hit this slow path with the mutex 
			for (int32 Index = 0; Index < GMallocBinnedAllocExtra && Pool->HasFreeBin(); Index++)
			{
				if (!Lists->Free(Result, PoolIndex, Table.BinSize))
				{
					break;
				}
				Result = Pool->AllocateBin(Table.BinSize);
				UE_MBC_UPDATE_STATS(++Table.TotalUsedBins);
			}
		}
	}
	if (!Pool->HasFreeBin())
	{
		Table.ExhaustedPools.LinkToFront(Pool);
	}

	return Result;
}

void* FMallocBinned2::MallocNoTLS(SIZE_T Size, uint32 Alignment)
{
	// Only allocate from the small pools if the size is small enough and the alignment isn't crazy large.
	// With large alignments, we'll waste a lot of memory allocating an entire page, but such alignments are highly unlikely in practice.
	bool bUseSmallPool = UseSmallAlloc(Size, Alignment);
	if (!bUseSmallPool)
	{
		bUseSmallPool = PromoteToLargerBin(Size, Alignment, *this);
	}

	if (bUseSmallPool)
	{
		const uint32 PoolIndex = BoundSizeToPoolIndex(Size, MemSizeToPoolIndex);

		NOALLOC_SCOPE_CYCLE_COUNTER(STAT_FMallocBinned2_MallocExternalSmall);

		// Allocate from small object pool.
		FPoolTable& Table = SmallPoolTables[PoolIndex];

		UE::TUniqueLock Lock(Table.Mutex);

		FPoolInfo* Pool;
		if (!Table.ActivePools.IsEmpty())
		{
			Pool = &Table.ActivePools.GetFrontPool();
		}
		else
		{
			Pool = &Table.ActivePools.PushNewPoolToFront(*this, Table, PoolIndex);
		}

		void* Result = Pool->AllocateBin(Table.BinSize);
#if UE_MB2_DOUBLE_FREE_DETECTION
		ClearDoubleFreeCanary(Result);
#endif
#if UE_MB2_ALLOCATOR_STATS
		++Table.TotalUsedBins;
		AllocatedSmallPoolMemory.fetch_add(Table.BinSize, std::memory_order_relaxed);
#endif

#if UE_MBC_LIGHTWEIGHT_BIN_CALLSTACK_TRACKER
		TrackBinAllocation(Table.BinSize, Table.TotalAllocatedMem);
#endif

		if (!Pool->HasFreeBin())
		{
			Table.ExhaustedPools.LinkToFront(Pool);
		}

		return Result;
	}
	return MallocExternalLarge(Size, Alignment);
}

void* FMallocBinned2::MallocExternalLarge(SIZE_T Size, uint32 Alignment)
{
	Alignment = FMath::Max<uint32>(Alignment, UE_MBC_STANDARD_ALIGNMENT);
	Size = Align(FMath::Max((SIZE_T)1, Size), Alignment);

	checkf(FMath::IsPowerOfTwo(Alignment), TEXT("Invalid Malloc alignment: '%u' is not a power of two"), Alignment);
	checkf(Alignment <= GMB2PageSize, TEXT("Invalid Malloc alignment: '%u' is greater than the page size '%u'"), Alignment, GMB2PageSize);
	
	const SIZE_T AlignedSize = Align(Size, OsAllocationGranularity);
	checkf(IsSupportedSize(AlignedSize), TEXT("Invalid Malloc size: '%" SIZE_T_FMT "'"), AlignedSize);

	LLM_PLATFORM_SCOPE(ELLMTag::FMalloc);

	FPoolInfo* Pool;
	void*      Result;
	{
		NOALLOC_SCOPE_CYCLE_COUNTER(STAT_FMallocBinned2_MallocExternalLarge);

		UE::TUniqueLock Lock(ExternalAllocMutex);

		// Use OS for non-pooled allocations.
		LogLargeAllocation(AlignedSize);
		Result = CachedOSPageAllocator.Allocate(AlignedSize, 0, &ExternalAllocMutex);
		if (!Result)
		{
			ExternalAllocMutex.Unlock();
			OutOfMemory(AlignedSize);  // OutOfMemory is fatal and does not return
		}

		Pool = Internal::GetOrCreatePoolInfo(*this, Result, FPoolInfo::ECanary::FirstFreeBlockIsOSAllocSize);
	}

	UE_CLOGF(!IsAligned(Result, Alignment) ,LogMemory, Fatal, "FMallocBinned2 alignment was too large for OS. Alignment=%d   Ptr=%p", Alignment, Result);
	check(IsAligned(Result, GMB2PageSize) && IsOSAllocation(Result));

#if UE_MB2_ALLOCATOR_STATS
	AllocatedLargePoolMemory.fetch_add(Size, std::memory_order_relaxed);
	AllocatedLargePoolMemoryWAlignment.fetch_add(AlignedSize, std::memory_order_relaxed);
#endif

	// Create pool.
	check(Size > 0 && AlignedSize >= OsAllocationGranularity);
	Pool->SetOSAllocationSizes(Size, AlignedSize);

	return Result;
}

void* FMallocBinned2::Realloc(void* Ptr, SIZE_T NewSize, uint32 Alignment)
{
	if (NewSize == 0)
	{
		FMallocBinned2::Free(Ptr);
		return nullptr;
	}

	checkf(FMath::IsPowerOfTwo(Alignment), TEXT("Invalid Realloc alignment: '%u' is not a power of two"), Alignment);

	if (!IsOSAllocation(Ptr))
	{
		check(Ptr); // null is 64k aligned so we should not be here
		// Reallocate to a smaller/bigger pool if necessary
		FPoolHeader* Header = GetPoolHeaderFromPointer(Ptr);
		CanaryTest(Header);
		const uint32 BinSize = Header->BinSize;
		const uint32 PoolIndex = Header->PoolIndex;
#if BINNED2_FORK_SUPPORT
		// If the canary is the pre-fork one, we should not allow this allocation to grow in-place to avoid copying a page from the parent process.
		if (Header->CanaryAndForkState == CurrentCanary &&
#else
		if( 
#endif
			((NewSize <= BinSize) & (IsAligned(BinSize, Alignment))) && // one branch, not two
			(PoolIndex == 0 || NewSize > PoolIndexToBinSize(PoolIndex - 1)))
		{
			return Ptr;
		}

		// Reallocate and copy the data across
		void* Result = FMallocBinned2::Malloc(NewSize, Alignment);
		FMemory::Memcpy(Result, Ptr, FMath::Min<SIZE_T>(NewSize, BinSize));
		FMallocBinned2::Free(Ptr);
		return Result;
	}
	if (!Ptr)
	{
		void* Result = FMallocBinned2::Malloc(NewSize, Alignment);
		return Result;
	}

	NOALLOC_SCOPE_CYCLE_COUNTER(STAT_FMallocBinned2_ReallocExternal);

	// Allocated from OS.
	ExternalAllocMutex.Lock();
	FPoolInfo* Pool = Internal::FindPoolInfo(*this, Ptr);
	UE_CLOGF(!Pool, LogMemory, Fatal, "FMallocBinned2 Attempt to realloc an unrecognized pointer %p", Ptr);

	const UPTRINT PoolOsBytes = Pool->GetOsAllocatedBytes();
	const SIZE_T PoolOSRequestedBytes = Pool->GetOSRequestedBytes();
	checkf(PoolOSRequestedBytes <= PoolOsBytes, TEXT("FMallocBinned2::Realloc %d %d"), int32(PoolOSRequestedBytes), int32(PoolOsBytes));
	bool bUseSmallMalloc = UseSmallAlloc(NewSize, Alignment);
	if (!bUseSmallMalloc)
	{
		bUseSmallMalloc = PromoteToLargerBin(NewSize, Alignment, *this);
	}

	if (NewSize > PoolOsBytes || // can't fit in the old block
		bUseSmallMalloc || // can switch to the small bin allocator
		Align(NewSize, OsAllocationGranularity) < PoolOsBytes) // we can get some pages back
	{
		// Grow or shrink.
		void* Result;
		if (bUseSmallMalloc)
		{
			// Unlock before a small alloc, which rarely takes a lock
			ExternalAllocMutex.Unlock();
			Result = MallocExternalSmall(NewSize, Alignment);
		}
		else
		{
			// Unlock after a large alloc, which does take a lock, to save unlocking and re-locking unnecessarily
			Result = MallocExternalLarge(NewSize, Alignment);
			ExternalAllocMutex.Unlock();
		}

		FMemory::Memcpy(Result, Ptr, FMath::Min(NewSize, PoolOSRequestedBytes));
		FMallocBinned2::Free(Ptr);
		return Result;
	}

	ExternalAllocMutex.Unlock();

	Alignment = FMath::Max<uint32>(Alignment, UE_MBC_STANDARD_ALIGNMENT);
	NewSize = Align(FMath::Max((SIZE_T)1, NewSize), Alignment);

	checkf(Alignment <= GMB2PageSize, TEXT("Invalid Realloc alignment: '%u' is greater than the page size '%u'"), Alignment, GMB2PageSize);
	checkf(IsSupportedSize(NewSize), TEXT("Invalid Realloc size: '%" SIZE_T_FMT "'"), NewSize);

	UE_MBC_UPDATE_STATS(AllocatedLargePoolMemory.fetch_add((int64)NewSize - (int64)PoolOSRequestedBytes, std::memory_order_relaxed));

	Pool->SetOSAllocationSizes(NewSize, PoolOsBytes);

	return Ptr;
}

void FMallocBinned2::Free(void* Ptr)
{
	NOALLOC_SCOPE_CYCLE_COUNTER(STAT_FMallocBinned2_FreeExternal);

	if (!IsOSAllocation(Ptr))
	{
		check(Ptr); // null is 64k aligned so we should not be here
		FPoolHeader* Header = GetPoolHeaderFromPointer(Ptr);
		CanaryTest(Header);

#if BINNED2_FORK_SUPPORT
		if (Header->CanaryAndForkState != CurrentCanary)
		{
			// This page was allocated before we forked so we want to avoid dirtying it by writing a linked list into it
			return;
		}
#endif
		const uint32 BinSize = Header->BinSize;
		const uint32 PoolIndex = Header->PoolIndex;

#if UE_MB2_DOUBLE_FREE_DETECTION
		SetAndValidateDoubleFreeCanary(Ptr);
#endif

		FBundleNode* BundlesToRecycle = nullptr;
		FPerThreadFreeBlockLists* Lists = GMallocBinnedPerThreadCaches ? FPerThreadFreeBlockLists::Get() : nullptr;
		if (Lists)
		{
			BundlesToRecycle = Lists->RecycleFullBundle(PoolIndex, Private::GGlobalRecycler);
			const bool bPushed = Lists->Free(Ptr, PoolIndex, BinSize);
			check(bPushed);
			UE_MBC_UPDATE_STATS(Lists->AllocatedMemory -= BinSize);
		}
		else
		{
			BundlesToRecycle = (FBundleNode*)Ptr;
			BundlesToRecycle->SetNextNodeInCurrentBundle(nullptr);
		}

		if (BundlesToRecycle)
		{
			Private::FreeBundles(*this, BundlesToRecycle, PoolIndex);
#if UE_MB2_ALLOCATOR_STATS
			if (!Lists)
			{
				// lists track their own stat track them instead in the global stat if we don't have lists
				AllocatedSmallPoolMemory.fetch_sub((int64)BinSize, std::memory_order_relaxed);
			}
#endif
		}
	}
	else if (Ptr)
	{
		UE::TUniqueLock Lock(ExternalAllocMutex);
		FPoolInfo* Pool = Internal::FindPoolInfo(*this, Ptr);
		if (!Pool)
		{
			UE_LOGF(LogMemory, Fatal, "FMallocBinned2 Attempt to free an unrecognized pointer %p", Ptr);
		}
		const UPTRINT PoolOsBytes = Pool->GetOsAllocatedBytes();
		const SIZE_T PoolOSRequestedBytes = Pool->GetOSRequestedBytes();

#if UE_MB2_ALLOCATOR_STATS
		AllocatedLargePoolMemory.fetch_sub((int64)PoolOSRequestedBytes, std::memory_order_relaxed);
		AllocatedLargePoolMemoryWAlignment.fetch_sub((int64)PoolOsBytes, std::memory_order_relaxed);
#endif

		checkf(PoolOSRequestedBytes <= PoolOsBytes, TEXT("FMallocBinned2::Free %d %d"), int32(PoolOSRequestedBytes), int32(PoolOsBytes));
		Pool->SetCanary(FPoolInfo::ECanary::Unassigned, true, false);
		// Free an OS allocation.
		CachedOSPageAllocator.Free(Ptr, PoolOsBytes, &ExternalAllocMutex, FPerThreadFreeBlockLists::Get() != nullptr && GBinned2MoveOSFreesOffTimeCriticalThreads != 0);
	}
}

void FMallocBinned2::FreeNoTLS(void* Ptr)
{
	NOALLOC_SCOPE_CYCLE_COUNTER(STAT_FMallocBinned2_FreeExternal);

	if (!IsOSAllocation(Ptr))
	{
		check(Ptr); // null is 64k aligned so we should not be here
		FPoolHeader* Header = GetPoolHeaderFromPointer(Ptr);
		CanaryTest(Header);

#if BINNED2_FORK_SUPPORT
		if (Header->CanaryAndForkState != CurrentCanary)
		{
			// This page was allocated before we forked so we want to avoid dirtying it by writing a linked list into it
			return;
		}
#endif

#if UE_MB2_DOUBLE_FREE_DETECTION
		SetAndValidateDoubleFreeCanary(Ptr);
#endif

		FBundleNode* BundlesToRecycle = (FBundleNode*)Ptr;
		BundlesToRecycle->SetNextNodeInCurrentBundle(nullptr);

		Private::FreeBundles(*this, BundlesToRecycle, Header->PoolIndex);
	}
	else if (Ptr)
	{
		Free(Ptr);
	}
}

void FMallocBinned2::FPoolList::ValidateActivePools() const
{
	for (const FPoolInfo* const* PoolPtr = &Front; *PoolPtr; PoolPtr = &(*PoolPtr)->Next)
	{
		const FPoolInfo* Pool = *PoolPtr;
		check(Pool->PtrToPrevNext == PoolPtr);
		check(Pool->FirstFreeBlock);
		for (FFreeBlock* Free = Pool->FirstFreeBlock; Free; Free = Free->GetNextFreeBlock())
		{
			check(Free->GetNumFreeBins() > 0);
		}
	}
}

void FMallocBinned2::FPoolList::ValidateExhaustedPools() const
{
	for (const FPoolInfo* const* PoolPtr = &Front; *PoolPtr; PoolPtr = &(*PoolPtr)->Next)
	{
		const FPoolInfo* Pool = *PoolPtr;
		check(Pool->PtrToPrevNext == PoolPtr);
		check(!Pool->FirstFreeBlock);
	}
}

bool FMallocBinned2::ValidateHeap()
{
	NOALLOC_SCOPE_CYCLE_COUNTER(STAT_FMallocBinned2_ValidateHeap);
	
	for (FPoolTable& Table : SmallPoolTables)
	{
		UE::TUniqueLock Lock(Table.Mutex);
		Table.ActivePools.ValidateActivePools();
		Table.ExhaustedPools.ValidateExhaustedPools();
	}

	return true;
}

const TCHAR* FMallocBinned2::GetDescriptiveName()
{
	return TEXT("Binned2");
}

void FMallocBinned2::FreeBundles(FBundleNode* Bundles, uint32 PoolIndex)
{
	Private::FreeBundles(*this, Bundles, PoolIndex);
}

void FMallocBinned2::Trim(bool bTrimThreadCaches)
{
	if (GMallocBinnedPerThreadCaches && bTrimThreadCaches)
	{
		FMallocBinnedCommonUtils::Trim(*this);

#if !UE_USE_VERYLARGEPAGEALLOCATOR
		UE::TUniqueLock Lock(ExternalAllocMutex);
		// this cache is recycled anyway, if you need to trim it based on being OOM, it's already too late.
		CachedOSPageAllocator.FreeAll(&ExternalAllocMutex);
#endif
		//UE_LOGF(LogTemp, Display, "Trim CachedOSPageAllocator = %6.2fms", 1000.0f * float(FPlatformTime::Seconds() - StartTime));
	}
}

void FMallocBinned2::FlushCurrentThreadCacheInternal(bool bNewEpochOnly)
{
	FMallocBinnedCommonUtils::FlushCurrentThreadCache(*this, bNewEpochOnly);
}

void FMallocBinned2::CanaryFail(const FPoolHeader* Header) const
{
#if BINNED2_FORK_SUPPORT
	UE_LOGF(LogMemory, Fatal, "FMallocBinned2 Attempt to realloc an unrecognized block %p   canary == 0x%x != 0x%x or 0x%x ", (void*)Header, (int32)Header->CanaryAndForkState, EnumToUnderlyingType(CurrentCanary), EnumToUnderlyingType(OldCanary));
#else
	UE_LOGF(LogMemory, Fatal, "FMallocBinned2 Attempt to realloc an unrecognized block %p   canary == 0x%x != 0x%x", (void*)Header, (int32)Header->CanaryAndForkState, int(CurrentCanary));
#endif
}

void FMallocBinned2::DumpAllocatorStats(class FOutputDevice& Ar)
{
#if UE_MB2_ALLOCATOR_STATS

	const int64  TotalAllocatedSmallPoolMemory           = GetTotalAllocatedSmallPoolMemory();
	const int64  LocalAllocatedLargePoolMemory           = AllocatedLargePoolMemory.load(std::memory_order_relaxed);
	const int64  LocalAllocatedLargePoolMemoryWAlignment = AllocatedLargePoolMemoryWAlignment.load(std::memory_order_relaxed);
	const uint64 OSPageAllocatorCachedFreeSize           = CachedOSPageAllocator.GetCachedFreeTotal();

	Ar.Logf(TEXT("FMallocBinned2 Mem report"));
	Ar.Logf(TEXT("Constants.BinnedPageSize = %d"), int32(GMB2PageSize));
	Ar.Logf(TEXT("Constants.BinnedAllocationGranularity = %d"), int32(OsAllocationGranularity));
	Ar.Logf(TEXT("Small Pool Allocations: %fmb  (including bin size padding)"), ((double)TotalAllocatedSmallPoolMemory) / (1024.0f * 1024.0f));
	Ar.Logf(TEXT("Small Pool OS Allocated: %fmb"), ((double)AllocatedOSSmallPoolMemory.load(std::memory_order_relaxed)) / (1024.0f * 1024.0f));
	Ar.Logf(TEXT("Large Pool Requested Allocations: %fmb"), ((double)AllocatedLargePoolMemory) / (1024.0f * 1024.0f));
	Ar.Logf(TEXT("Large Pool OS Allocated: %fmb"), ((double)AllocatedLargePoolMemoryWAlignment) / (1024.0f * 1024.0f));
	Ar.Logf(TEXT("Requested Allocations: %fmb"), ((double)LocalAllocatedLargePoolMemory) / (1024.0f * 1024.0f));
	Ar.Logf(TEXT("OS Allocated: %fmb"), ((double)LocalAllocatedLargePoolMemoryWAlignment) / (1024.0f * 1024.0f));
	Ar.Logf(TEXT("PoolInfo: %fmb"), ((double)PoolInfoMemory) / (1024.0f * 1024.0f));
	Ar.Logf(TEXT("Hash: %fmb"), ((double)Binned2HashMemory) / (1024.0f * 1024.0f));
	Ar.Logf(TEXT("TLS: %fmb"), ((double)TLSMemory.load(std::memory_order_relaxed)) / (1024.0f * 1024.0f));
	Ar.Logf(TEXT("Total allocated from OS: %fmb"),
		((double)
			AllocatedOSSmallPoolMemory + AllocatedLargePoolMemoryWAlignment + PoolInfoMemory + Binned2HashMemory + TLSMemory.load(std::memory_order_relaxed)
			) / (1024.0f * 1024.0f));
	Ar.Logf(TEXT("Cached free OS pages: %fmb"), ((double)OSPageAllocatorCachedFreeSize) / (1024.0f * 1024.0f));

	for (int32 i = 0; i < UE_MB2_SMALL_POOL_COUNT; i++)
	{
		const float Fragmentation = (SmallPoolTables[i].TotalAllocatedBins > 0) ? 1.0f - (float)SmallPoolTables[i].TotalUsedBins / (float)SmallPoolTables[i].TotalAllocatedBins : 0;
		const float TotalMem = (float)SmallPoolTables[i].TotalAllocatedMem / 1024.0f / 1024.0f;
		Ar.Logf(TEXT("Bin %6d Fragmentation %d %%, Wasted Mem %.2f MB, Total Allocated Mem %.2f MB"),
				PoolIndexToBinSize(i), int(Fragmentation * 100.0f), TotalMem * Fragmentation, TotalMem);
	}

#if !PLATFORM_UNIX && !PLATFORM_ANDROID
	// Doesn't implemented
#else
	CachedOSPageAllocator.DumpAllocatorStats(Ar);
#endif

#else
	Ar.Logf(TEXT("Allocator Stats for binned2 are not in this build. Define UE_MBC_ALLOCATOR_STATS for your project"));
#endif
}

void FMallocBinned2::UpdateStats()
{
	//Report total cached free memory in the COSPA and separately report memory that can be immediately freed back the kernel at any time
	CSV_CUSTOM_STAT(FMemory, AllocatorCachedSlackMB, (int32)(CachedOSPageAllocator.GetCachedFreeTotal() / (1024 * 1024)), ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT(FMemory, AllocatorImmediatelyFreeableCachedSlackMB, (int32)(CachedOSPageAllocator.GetCachedImmediatelyFreeable() / (1024 * 1024)), ECsvCustomStatOp::Set);

	FMallocBinnedCommonUtils::UpdateStatsCommon(*this);
	CachedOSPageAllocator.UpdateStats();
	FScopedVirtualMallocTimer::UpdateStats();
}

void* FMallocBinned2::AllocateMetaDataMemory(SIZE_T Size)
{
	LLM_PLATFORM_SCOPE(ELLMTag::FMalloc);
	Size = Align(Size, OsAllocationGranularity);
	return FPlatformMemory::BinnedAllocFromOS(Size);
}

void FMallocBinned2::FreeMetaDataMemory(void* Ptr, SIZE_T Size)
{
	if (Ptr)
	{
		Size = Align(Size, FMallocBinned2::OsAllocationGranularity);
		FPlatformMemory::BinnedFreeToOS(Ptr, Size);
	}
}


PRAGMA_RESTORE_UNSAFE_TYPECAST_WARNINGS
