// Copyright Epic Games, Inc. All Rights Reserved.

#include "HAL/MallocBinned3.h"

PRAGMA_DISABLE_UNSAFE_TYPECAST_WARNINGS

#if PLATFORM_HAS_FPlatformVirtualMemoryBlock
#include "Async/UniqueLock.h"
#include "Templates/Function.h"
#include "GenericPlatform/GenericPlatformProcess.h"
#include "Stats/Stats.h"
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformMisc.h"
#include "HAL/MallocBinnedCommonUtils.h"

#if UE_MB3_USE_CACHED_PAGE_ALLOCATOR_FOR_LARGE_ALLOCS
#	include "HAL/Allocators/CachedOSPageAllocator.h"
#	define UE_MB3_MAX_CACHED_OS_FREES (64)
#	define UE_MB3_MAX_CACHED_OS_FREES_BYTE_LIMIT (64*1024*1024)

	enum class EMB3CacheState : int8
	{
		Enabled = 0,
		Disabled = 1,
		Trimmed = 2,
	};

	std::atomic<EMB3CacheState> LargeAllocsCacheEnabledState(EMB3CacheState::Enabled);

	typedef TCachedOSPageAllocator<UE_MB3_MAX_CACHED_OS_FREES, UE_MB3_MAX_CACHED_OS_FREES_BYTE_LIMIT> TBinned3CachedOSPageAllocator;

	TBinned3CachedOSPageAllocator& GetCachedOSPageAllocator()
	{
		static TBinned3CachedOSPageAllocator Singleton;
		return Singleton;
	}
#endif	// ~UE_MB3_USE_CACHED_PAGE_ALLOCATOR_FOR_LARGE_ALLOCS

#if UE_MB3_ALLOCATOR_STATS
	TAtomic<int64> Binned3Commits;
	TAtomic<int64> Binned3Decommits;

	int64 Binned3FreeBitsMemory = 0;
#endif

#define UE_MB3_TIME_LARGE_BLOCKS (0)

#if UE_MB3_TIME_LARGE_BLOCKS
	TAtomic<double> MemoryRangeReserveTotalTime(0.0);
	TAtomic<int32> MemoryRangeReserveTotalCount(0);

	TAtomic<double> MemoryRangeFreeTotalTime(0.0);
	TAtomic<int32> MemoryRangeFreeTotalCount(0);
#endif

uint16 FMallocBinned3::SmallBinSizesShifted[UE_MB3_SMALL_POOL_COUNT + 1] = { 0 };
uint8* FMallocBinned3::Binned3BaseVMPtr = nullptr;
FMallocBinned3* FMallocBinned3::MallocBinned3 = nullptr;
// Mapping of sizes to small table indices
uint8 FMallocBinned3::MemSizeToPoolIndex[SIZE_TO_POOL_INDEX_NUM] = { 0 };

template <>
thread_local FMallocBinned3::FPerThreadFreeBlockLists* FMallocBinned3::FPerThreadFreeBlockLists::ThreadSingleton = nullptr;

template <class T>
static void SetCanaryInternal(T& PoolInfo, typename T::ECanary ShouldBe, bool bPreexisting, bool bGuarnteedToBeNew)
{
	if (bPreexisting)
	{
		if (bGuarnteedToBeNew)
		{
			UE_LOGF(LogMemory, Fatal, "MallocBinned3 Corruption Canary was 0x%x, should be 0x%x. This block is both preexisting and guaranteed to be new; which makes no sense.", int32(PoolInfo.Canary), int32(ShouldBe));
		}
		if (ShouldBe == T::ECanary::Unassigned)
		{
			if (PoolInfo.Canary != T::ECanary::Assigned)
			{
				UE_LOGF(LogMemory, Fatal, "MallocBinned3 Corruption Canary was 0x%x, will be 0x%x because this block should be preexisting and in use.", int32(PoolInfo.Canary), int32(ShouldBe));
			}
		}
		else if (PoolInfo.Canary != ShouldBe)
		{
			UE_LOGF(LogMemory, Fatal, "MallocBinned3 Corruption Canary was 0x%x, should be 0x%x because this block should be preexisting.", int32(PoolInfo.Canary), int32(ShouldBe));
		}
	}
	else
	{
		if (bGuarnteedToBeNew)
		{
			if (PoolInfo.Canary != T::ECanary::Unassigned)
			{
				UE_LOGF(LogMemory, Fatal, "MallocBinned3 Corruption Canary was 0x%x, will be 0x%x. This block is guaranteed to be new yet is it already assigned.", int32(PoolInfo.Canary), int32(ShouldBe));
			}
		}
		else if (PoolInfo.Canary != ShouldBe && PoolInfo.Canary != T::ECanary::Unassigned)
		{
			UE_LOGF(LogMemory, Fatal, "MallocBinned3 Corruption Canary was 0x%x, will be 0x%x does not have an expected value.", int32(PoolInfo.Canary), int32(ShouldBe));
		}
	}
	PoolInfo.Canary = ShouldBe;
}

/** Information about a piece of free memory. */
struct FFreeBlock
{
	enum
	{
		CANARY_VALUE = 0xe7
	};

	FORCEINLINE FFreeBlock(uint32 InBlockSize, uint32 InBinSize, uint8 InPoolIndex)
		: BinSizeShifted(InBinSize >> UE_MBC_BIN_SIZE_SHIFT)
		, PoolIndex(InPoolIndex)
		, Canary(CANARY_VALUE)
		, NextFreeBlockIndex(InvalidNextFreeBlock)
	{
		check(InPoolIndex < MAX_uint8 && (InBinSize >> UE_MBC_BIN_SIZE_SHIFT) <= MAX_uint16);
		NumFreeBins = InBlockSize / InBinSize;
	}

	FORCEINLINE uint32 GetNumFreeBins() const
	{
		return NumFreeBins;
	}

	FORCEINLINE bool IsCanaryOk() const
	{
		return Canary == FFreeBlock::CANARY_VALUE;
	}

	FORCEINLINE void CanaryTest() const
	{
		if (!IsCanaryOk())
		{
			CanaryFail();
		}
		//checkSlow(PoolIndex == BoundSizeToPoolIndex(uint32(BinSizeShifted) << UE_MBC_BIN_SIZE_SHIFT));
	}

	FORCEINLINE void CanaryFail() const
	{
		UE_LOGF(LogMemory, Fatal, "FMallocBinned3 Attempt to realloc an unrecognized pointer %p   canary == 0x%x != 0x%x", (void*)this, (int32)Canary, (int32)CANARY_VALUE);
	}

	FORCEINLINE void* AllocateBin()
	{
		--NumFreeBins;
		return (uint8*)this + NumFreeBins * (uint32(BinSizeShifted) << UE_MBC_BIN_SIZE_SHIFT);
	}

	constexpr static uint32 MaxBitsForBinSizeShifted = 32 - FMath::CountLeadingZeros(UE_MB3_MAX_SMALL_POOL_SIZE >> UE_MBC_BIN_SIZE_SHIFT);
	constexpr static uint32 MaxBitsForPoolIndex = 32 - FMath::CountLeadingZeros(UE_MB3_SMALL_POOL_COUNT);
	constexpr static uint32 MaxBitsForFreeBins = 32 - FMath::CountLeadingZeros(UE_MBC_MAX_SUPPORTED_PLATFORM_PAGE_SIZE / UE_MBC_MIN_BIN_SIZE);	// max supported page size is 16 kb (on iOS) and the highest amount of free bins in one block will be for the 8 bytes bin in the 16kb block
	constexpr static uint32 InvalidNextFreeBlock = 1 << MaxBitsForFreeBins;

	uint32 BinSizeShifted : MaxBitsForBinSizeShifted;	// Size of the bins that this list points to >> UE_MBC_BIN_SIZE_SHIFT
	uint32 PoolIndex : MaxBitsForPoolIndex;				// Index of this pool
	uint32 Canary : 8;									// Constant value of 0xe7
	uint32 NumFreeBins : MaxBitsForFreeBins;			// Number of consecutive free bins here, at least 1 for a newly initialized FFreeBlock
	uint32 NextFreeBlockIndex : MaxBitsForFreeBins + 1;	// Next free FFreeBlock or InvalidNextFreeBlock
};

static_assert(sizeof(FFreeBlock) <= UE_MBC_MIN_BIN_SIZE, "FFreeBlock should fit into 8 bytes to support 8 bytes bins");


struct FMallocBinned3::FPoolInfoSmall		//This is more like BlockInfoSmall as it stores info per block
{
	enum ECanary
	{
		Unassigned = 0x3,
		Assigned = 0x1
	};

	uint32 Canary : 2;
	uint32 Taken : 15;
	uint32 NoFirstFreeIndex : 1;	// if set to 0 means there are free bins and we can lookup FirstFreeIndex, otherwise means the block is exhausted
	uint32 FirstFreeIndex : 14;

	FPoolInfoSmall()
		: Canary(ECanary::Unassigned)
		, Taken(0)
		, NoFirstFreeIndex(1)
		, FirstFreeIndex(0)
	{
		static_assert(sizeof(FPoolInfoSmall) == 4, "Padding fail");
	}

	void CheckCanary(ECanary ShouldBe) const
	{
		if (Canary != ShouldBe)
		{
			UE_LOGF(LogMemory, Fatal, "MallocBinned3 Corruption Canary was 0x%x, should be 0x%x", int32(Canary), int32(ShouldBe));
		}
	}

	void SetCanary(ECanary ShouldBe, bool bPreexisting, bool bGuarnteedToBeNew)
	{
		SetCanaryInternal(*this, ShouldBe, bPreexisting, bGuarnteedToBeNew);
	}

	bool HasFreeBin() const
	{
		CheckCanary(ECanary::Assigned);
		return !NoFirstFreeIndex;
	}

	void* AllocateBin(uint8* BlockPtr, uint32 BinSize)
	{
		check(HasFreeBin());
		++Taken;
		check(Taken != 0);
		FFreeBlock* Free = (FFreeBlock*)(BlockPtr + BinSize * FirstFreeIndex);
		void* Result = Free->AllocateBin();
		if (Free->GetNumFreeBins() == 0)
		{
			if (Free->NextFreeBlockIndex == FFreeBlock::InvalidNextFreeBlock)
			{
				FirstFreeIndex = 0;
				NoFirstFreeIndex = 1;
			}
			else
			{
				FirstFreeIndex = Free->NextFreeBlockIndex;
				check(uint32(FirstFreeIndex) == Free->NextFreeBlockIndex);
				check(((FFreeBlock*)(BlockPtr + BinSize * FirstFreeIndex))->GetNumFreeBins());
			}
		}

		return Result;
	}
};

FMallocBinned3::FPoolInfo::FPoolInfo()
	: Canary(ECanary::Unassigned)
	, AllocSize(0)
	, VMSizeDivVirtualSizeAlignment(0)
	, CommitSize(0)
{
}

void FMallocBinned3::FPoolInfo::CheckCanary(ECanary ShouldBe) const
{
	if (Canary != ShouldBe)
	{
		UE_LOGF(LogMemory, Fatal, "MallocBinned3 Corruption Canary was 0x%x, should be 0x%x", int32(Canary), int32(ShouldBe));
	}
}

void FMallocBinned3::FPoolInfo::SetCanary(ECanary ShouldBe, bool bPreexisting, bool bGuarnteedToBeNew)
{
	SetCanaryInternal(*this, ShouldBe, bPreexisting, bGuarnteedToBeNew);
}

uint32 FMallocBinned3::FPoolInfo::GetOSRequestedBytes() const
{
	return AllocSize;
}

uint32 FMallocBinned3::FPoolInfo::GetOsCommittedBytes() const
{
	return CommitSize;
}

uint32 FMallocBinned3::FPoolInfo::GetOsVMPages() const
{
	CheckCanary(ECanary::Assigned);
	return VMSizeDivVirtualSizeAlignment;
}

void FMallocBinned3::FPoolInfo::SetOSAllocationSize(uint32 InRequestedBytes)
{
	CheckCanary(ECanary::Assigned);
	AllocSize = InRequestedBytes;
	check(AllocSize > 0 && CommitSize >= AllocSize && VMSizeDivVirtualSizeAlignment * FPlatformMemory::FPlatformVirtualMemoryBlock::GetVirtualSizeAlignment() >= CommitSize);
}

void FMallocBinned3::FPoolInfo::SetOSAllocationSizes(uint32 InRequestedBytes, UPTRINT InCommittedBytes, uint32 InVMSizeDivVirtualSizeAlignment)
{
	CheckCanary(ECanary::Assigned);
	AllocSize = InRequestedBytes;
	CommitSize = InCommittedBytes;
	VMSizeDivVirtualSizeAlignment = InVMSizeDivVirtualSizeAlignment;
	check(AllocSize > 0 && CommitSize >= AllocSize && VMSizeDivVirtualSizeAlignment * FPlatformMemory::FPlatformVirtualMemoryBlock::GetVirtualSizeAlignment() >= CommitSize);
}

struct FMallocBinned3::Private
{
	/**
	* Gets the FPoolInfoSmall for a small block memory address. If no valid info exists one is created.
	*/
	static FPoolInfoSmall* GetOrCreatePoolInfoSmall(FMallocBinned3& Allocator, uint32 InPoolIndex, uint32 BlockIndex)
	{
		const uint32 InfosPerPage = Allocator.SmallPoolInfosPerPlatformPage;
		const uint32 InfoOuterIndex = BlockIndex / InfosPerPage;
		const uint32 InfoInnerIndex = BlockIndex % InfosPerPage;
		FPoolInfoSmall*& InfoBlock = Allocator.SmallPoolTables[InPoolIndex].PoolInfos[InfoOuterIndex];
		if (!InfoBlock)
		{
			InfoBlock = (FPoolInfoSmall*)Allocator.AllocateMetaDataMemory(Allocator.OsAllocationGranularity);
			UE_MBC_UPDATE_STATS(PoolInfoMemory += Allocator.OsAllocationGranularity);

			DefaultConstructItems<FPoolInfoSmall>((void*)InfoBlock, InfosPerPage);
		}

		FPoolInfoSmall* Result = &InfoBlock[InfoInnerIndex];

		bool bGuaranteedToBeNew = false;
		if (BlockIndex >= Allocator.SmallPoolTables[InPoolIndex].NumEverUsedBlocks)
		{
			bGuaranteedToBeNew = true;
			Allocator.SmallPoolTables[InPoolIndex].NumEverUsedBlocks = BlockIndex + 1;
		}
		Result->SetCanary(FPoolInfoSmall::ECanary::Assigned, false, bGuaranteedToBeNew);
		return Result;
	}

	static MallocBinnedPrivate::TGlobalRecycler<UE_MB3_SMALL_POOL_COUNT> GGlobalRecycler;

	static void FreeBundles(FMallocBinned3& Allocator, FBundleNode* BundlesToRecycle, uint32 InPoolIndex)
	{
		FPoolTable& Table = Allocator.SmallPoolTables[InPoolIndex];
		UE::TUniqueLock Lock(Table.Mutex);

		FBundleNode* Node = BundlesToRecycle;
		do
		{
			FBundleNode* NextNode = Node->GetNextNodeInCurrentBundle();

			uint32 OutBlockIndex;
			const uint32 BinSize = Allocator.SmallPoolTables[InPoolIndex].BinSize;
			void* BaseBlockPtr = Allocator.BlockPointerFromContainedPtr(Node, Allocator.SmallPoolTables[InPoolIndex].NumMemoryPagesPerBlock, OutBlockIndex);
			const uint32 BinIndexWithinBlock = (((uint8*)Node) - ((uint8*)BaseBlockPtr)) / BinSize;

			FPoolInfoSmall* NodePoolBlock = Allocator.SmallPoolTables[InPoolIndex].PoolInfos[OutBlockIndex / Allocator.SmallPoolInfosPerPlatformPage];
			if (!NodePoolBlock)
			{
				UE_LOGF(LogMemory, Fatal, "FMallocBinned3 Attempt to free an unrecognized small block %p", Node);
			}
			FPoolInfoSmall* NodePool = &NodePoolBlock[OutBlockIndex % Allocator.SmallPoolInfosPerPlatformPage];

			NodePool->CheckCanary(FPoolInfoSmall::ECanary::Assigned);

			const bool bWasExhaused = NodePool->NoFirstFreeIndex;

			// Free a pooled allocation.
			FFreeBlock* Free = (FFreeBlock*)Node;
			Free->NumFreeBins = 1;
			Free->NextFreeBlockIndex = NodePool->NoFirstFreeIndex ? FFreeBlock::InvalidNextFreeBlock : NodePool->FirstFreeIndex;
			Free->BinSizeShifted = (BinSize >> UE_MBC_BIN_SIZE_SHIFT);
			Free->Canary = FFreeBlock::CANARY_VALUE;
			Free->PoolIndex = InPoolIndex;
			NodePool->FirstFreeIndex = BinIndexWithinBlock;
			NodePool->NoFirstFreeIndex = 0;
			check(uint32(NodePool->FirstFreeIndex) == BinIndexWithinBlock);

			UE_MBC_UPDATE_STATS(--Table.TotalUsedBins);

			// Free this pool.
			check(NodePool->Taken >= 1);
			if (--NodePool->Taken == 0)
			{
				NodePool->SetCanary(FPoolInfoSmall::ECanary::Unassigned, true, false);
				Table.BlocksAllocatedBits.FreeBit(OutBlockIndex);

				if (!bWasExhaused)
				{
					Table.BlocksExhaustedBits.AllocBit(OutBlockIndex);
				}

				const uint64 BlockSize = Allocator.SmallPoolTables[InPoolIndex].BlockSize;
				Allocator.Decommit(InPoolIndex, BaseBlockPtr, BlockSize);

#if UE_MB3_ALLOCATOR_STATS
				Table.TotalAllocatedBins -= BlockSize / BinSize;
				Table.TotalAllocatedMem -= BlockSize;
				AllocatedOSSmallPoolMemory.fetch_sub(BlockSize, std::memory_order_relaxed);
#endif
			}
			else if (bWasExhaused)
			{
				Table.BlocksExhaustedBits.FreeBit(OutBlockIndex);
			}

			Node = NextNode;
		} while (Node);
	}
};

MallocBinnedPrivate::TGlobalRecycler<UE_MB3_SMALL_POOL_COUNT> FMallocBinned3::Private::GGlobalRecycler;

void FMallocBinned3::FreeBundles(FBundleNode* Bundles, uint32 PoolIndex)
{
	Private::FreeBundles(*this, Bundles, PoolIndex);
}

FMallocBinned3::FPoolInfoSmall* FMallocBinned3::PushNewPoolToFront(FMallocBinned3::FPoolTable& Table, uint32 InBinSize, uint32 InPoolIndex, uint32& OutBlockIndex)
{
	// Allocate memory.
	const uint32 BlockIndex = Table.BlocksAllocatedBits.AllocBit();
	if (BlockIndex == MAX_uint32)
	{
		return nullptr;
	}
	uint8* FreePtr = BlockPointerFromIndecies(InPoolIndex, BlockIndex, Table.BlockSize);

	LLM_PLATFORM_SCOPE(ELLMTag::FMalloc);
	Commit(InPoolIndex, FreePtr, Table.BlockSize);
	const uint64 EndOffset = UPTRINT(FreePtr + Table.BlockSize) - UPTRINT(PoolBasePtr(InPoolIndex));
	if (EndOffset > Table.UnusedAreaOffsetLow)
	{
		Table.UnusedAreaOffsetLow = EndOffset;
	}
	FFreeBlock* Free = new ((void*)FreePtr) FFreeBlock(Table.BlockSize, InBinSize, InPoolIndex);
#if UE_MB3_ALLOCATOR_STATS
	AllocatedOSSmallPoolMemory.fetch_add((int64)Table.BlockSize, std::memory_order_relaxed);
	Table.TotalAllocatedBins += Free->NumFreeBins;
	Table.TotalAllocatedMem += Table.BlockSize;
#endif
	check(IsAligned(Free, OsAllocationGranularity));
	// Create pool
	FPoolInfoSmall* Result = Private::GetOrCreatePoolInfoSmall(*this, InPoolIndex, BlockIndex);
	Result->CheckCanary(FPoolInfoSmall::ECanary::Assigned);
	Result->Taken = 0;
	Result->FirstFreeIndex = 0;
	Result->NoFirstFreeIndex = 0;
	Table.BlocksExhaustedBits.FreeBit(BlockIndex);

	OutBlockIndex = BlockIndex;

	return Result;
}

FMallocBinned3::FPoolInfoSmall* FMallocBinned3::GetFrontPool(FPoolTable& Table, uint32 InPoolIndex, uint32& OutBlockIndex)
{
	OutBlockIndex = Table.BlocksExhaustedBits.NextAllocBit();
	if (OutBlockIndex == MAX_uint32)
	{
		return nullptr;
	}
	return Private::GetOrCreatePoolInfoSmall(*this, InPoolIndex, OutBlockIndex);
}

FMallocBinned3::FMallocBinned3(const FPlatformMemory::FPlatformVirtualMemoryBlock* ExternalMemoryBlock)
{
	static bool bOnce = false;
	check(!bOnce); // this is now a singleton-like thing and you cannot make multiple copies
	bOnce = true;

	OsAllocationGranularity = FPlatformMemory::FPlatformVirtualMemoryBlock::GetCommitAlignment();
	checkf(FMath::IsPowerOfTwo(OsAllocationGranularity), TEXT("OS page size must be a power of two"));

	// First thing we try to allocate address space for bins as it might help us to move forward Constants.AddressStart and reduce the amount of available address space for the Large OS Allocs
	// Available address space is used to reserve hash map that can address all of that range, so less addressable space means less memory is allocated for book keeping
	Binned3BaseVMBlock = ExternalMemoryBlock ? *ExternalMemoryBlock : FPlatformMemory::FPlatformVirtualMemoryBlock::AllocateVirtual(UE_MB3_SMALL_POOL_COUNT * UE_MB3_MAX_MEMORY_PER_POOL_SIZE, OsAllocationGranularity);
	check(Binned3BaseVMBlock.GetActualSize() >= UE_MB3_SMALL_POOL_COUNT * UE_MB3_MAX_MEMORY_PER_POOL_SIZE);
	Binned3BaseVMPtr = (uint8*)Binned3BaseVMBlock.GetVirtualPointer();
	check(IsAligned(Binned3BaseVMPtr, OsAllocationGranularity));
	verify(Binned3BaseVMPtr);

	FGenericPlatformMemoryConstants Constants = FPlatformMemory::GetConstants();
	if (Constants.AddressStart == (uint64)Binned3BaseVMPtr)
	{
		Constants.AddressStart += Align(UE_MB3_SMALL_POOL_COUNT * UE_MB3_MAX_MEMORY_PER_POOL_SIZE, OsAllocationGranularity);
	}

	// large slab sizes are possible OsAllocationGranularity = 65536;
	NumPoolsPerPage = OsAllocationGranularity / sizeof(FPoolInfo);
	check(OsAllocationGranularity % sizeof(FPoolInfo) == 0);  // these need to divide evenly!
	PtrToPoolMapping.Init(OsAllocationGranularity, NumPoolsPerPage, Constants.AddressStart, Constants.AddressLimit);

	checkf(Constants.AddressLimit > OsAllocationGranularity, TEXT("OS address limit must be greater than the page size")); // Check to catch 32 bit overflow in AddressLimit
	static_assert(UE_MB3_SMALL_POOL_COUNT <= 256, "Small bins size array size must fit in a byte");
	static_assert(sizeof(FFreeBlock) <= UE_MBC_MIN_BIN_SIZE, "Free block struct must be small enough to fit into the smallest bin");

	// Init pool tables.
	FSizeTableEntry SizeTable[UE_MB3_SMALL_POOL_COUNT];

	verify(FSizeTableEntry::FillSizeTable(OsAllocationGranularity, SizeTable, UE_MB3_BASE_PAGE_SIZE, UE_MB3_MAX_SMALL_POOL_SIZE, UE_MB3_BASE_PAGE_SIZE) == UE_MB3_SMALL_POOL_COUNT);
	checkf(SizeTable[UE_MB3_SMALL_POOL_COUNT - 1].BinSize == UE_MB3_MAX_SMALL_POOL_SIZE, TEXT("UE_MB3_MAX_SMALL_POOL_SIZE must be equal to the largest bin size"));
	checkf(sizeof(FFreeBlock)  <= SizeTable[0].BinSize, TEXT("Pool header must be able to fit into the smallest bin"));

	SmallPoolInfosPerPlatformPage = OsAllocationGranularity / sizeof(FPoolInfoSmall);

	uint32 RequiredMetaMem = 0;
	for (uint32 Index = 0; Index < UE_MB3_SMALL_POOL_COUNT; ++Index)
	{
		checkf(Index == 0 || SizeTable[Index - 1].BinSize < SizeTable[Index].BinSize, TEXT("Small bin sizes must be strictly increasing"));

		SmallPoolTables[Index].BinSize = SizeTable[Index].BinSize;
		SmallPoolTables[Index].NumMemoryPagesPerBlock = SizeTable[Index].NumMemoryPagesPerBlock;
		SmallPoolTables[Index].BlockSize = SizeTable[Index].NumMemoryPagesPerBlock * OsAllocationGranularity;

		SmallBinSizesShifted[Index + 1] = (SizeTable[Index].BinSize >> UE_MBC_BIN_SIZE_SHIFT);

		const int64 TotalNumberOfBlocks = UE_MB3_MAX_MEMORY_PER_POOL_SIZE / SmallPoolTables[Index].BlockSize;
		const uint32 Size = Align(sizeof(FPoolInfoSmall**) * (TotalNumberOfBlocks + SmallPoolInfosPerPlatformPage - 1) / SmallPoolInfosPerPlatformPage, PLATFORM_CACHE_LINE_SIZE);
		RequiredMetaMem += Size;

		const int64 AllocationSize = Align(FBitTree::GetMemoryRequirements(TotalNumberOfBlocks), PLATFORM_CACHE_LINE_SIZE);
		RequiredMetaMem += AllocationSize * 2;

#if UE_MB3_ALLOCATOR_STATS
		PoolInfoMemory += Size;
		Binned3FreeBitsMemory += AllocationSize * 2;
#endif
	}

	RequiredMetaMem = Align(RequiredMetaMem, OsAllocationGranularity);
	uint8* MetaMem = (uint8*)AllocateMetaDataMemory(RequiredMetaMem);
	const uint8* MetaMemEnd = MetaMem + RequiredMetaMem;
	FMemory::Memzero(MetaMem, RequiredMetaMem);

	for (uint32 Index = 0; Index < UE_MB3_SMALL_POOL_COUNT; ++Index)
	{
		const int64 TotalNumberOfBlocks = UE_MB3_MAX_MEMORY_PER_POOL_SIZE / SmallPoolTables[Index].BlockSize;
		const uint32 Size = Align(sizeof(FPoolInfoSmall**) * (TotalNumberOfBlocks + SmallPoolInfosPerPlatformPage - 1) / SmallPoolInfosPerPlatformPage, PLATFORM_CACHE_LINE_SIZE);

		SmallPoolTables[Index].PoolInfos = (FPoolInfoSmall**)MetaMem;
		MetaMem += Size;

		const int64 AllocationSize = Align(FBitTree::GetMemoryRequirements(TotalNumberOfBlocks), PLATFORM_CACHE_LINE_SIZE);
		SmallPoolTables[Index].BlocksAllocatedBits.FBitTreeInit(TotalNumberOfBlocks, MetaMem, AllocationSize, false);
		MetaMem += AllocationSize;

		SmallPoolTables[Index].BlocksExhaustedBits.FBitTreeInit(TotalNumberOfBlocks, MetaMem, AllocationSize, true);
		MetaMem += AllocationSize;
	}
	check(MetaMem <= MetaMemEnd);

	// Set up pool mappings
	uint8* IndexEntry = MemSizeToPoolIndex;
	uint32 PoolIndex  = 0;
	for (uint32 Index = 0; Index != SIZE_TO_POOL_INDEX_NUM; ++Index)
	{
		const uint32 BinSize = Index << UE_MBC_BIN_SIZE_SHIFT; // inverse of int32 Index = int32((Size >> UE_MBC_BIN_SIZE_SHIFT));
		while (SizeTable[PoolIndex].BinSize < BinSize)
		{
			++PoolIndex;
			check(PoolIndex != UE_MB3_SMALL_POOL_COUNT);
		}
		check(PoolIndex < 256);
		*IndexEntry++ = uint8(PoolIndex);
	}

	AllocateHashBuckets();

	MallocBinned3 = this;
	GFixedMallocLocationPtr = (FMalloc**)(&MallocBinned3);
}

FMallocBinned3::~FMallocBinned3()
{
}

void FMallocBinned3::Commit(uint32 InPoolIndex, void *Ptr, SIZE_T Size)
{
	UE_MBC_UPDATE_STATS(Binned3Commits++);
	Binned3BaseVMBlock.CommitByPtr(Ptr, Size);
	LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelAlloc(ELLMTracker::Platform, Ptr, Size));
}

void FMallocBinned3::Decommit(uint32 InPoolIndex, void *Ptr, SIZE_T Size)
{
	LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelFree(ELLMTracker::Platform, Ptr));
	UE_MBC_UPDATE_STATS(Binned3Decommits++);
	Binned3BaseVMBlock.DecommitByPtr(Ptr, Size);
}

void* FMallocBinned3::AllocateMetaDataMemory(SIZE_T Size)
{
	LLM_PLATFORM_SCOPE(ELLMTag::FMalloc);
	const size_t VirtualAlignedSize = Align(Size, FPlatformMemory::FPlatformVirtualMemoryBlock::GetVirtualSizeAlignment());
	FPlatformMemory::FPlatformVirtualMemoryBlock Block = FPlatformMemory::FPlatformVirtualMemoryBlock::AllocateVirtual(VirtualAlignedSize);
	const size_t CommitAlignedSize = Align(Size, FPlatformMemory::FPlatformVirtualMemoryBlock::GetCommitAlignment());
	Block.Commit(0, CommitAlignedSize);
	LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelAlloc(ELLMTracker::Platform, Block.GetVirtualPointer(), CommitAlignedSize));
	return Block.GetVirtualPointer();
}

void FMallocBinned3::FreeMetaDataMemory(void *Ptr, SIZE_T InSize)
{
	if (Ptr)
	{
		LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelFree(ELLMTracker::Platform, Ptr));

		InSize = Align(InSize, FPlatformMemory::FPlatformVirtualMemoryBlock::GetVirtualSizeAlignment());
		FPlatformMemory::FPlatformVirtualMemoryBlock Block(Ptr, InSize / FPlatformMemory::FPlatformVirtualMemoryBlock::GetVirtualSizeAlignment());
		Block.FreeVirtual();
	}
}

bool FMallocBinned3::IsInternallyThreadSafe() const
{ 
	return true;
}

void* FMallocBinned3::Malloc(SIZE_T Size, uint32 Alignment)
{
	// Fast path: Allocate from the small pools if the size is small enough and the alignment <= binned3 min alignment.
	//            Larger alignments can waste a lot of memory allocating an entire page, so some smaller alignments are 
	//			  handled in the fallback path if less than a predefined max small pool alignment. 

	bool UsePools = (Size <= UE_MB3_MAX_SMALL_POOL_SIZE);

	check(FMath::IsPowerOfTwo(Alignment));

	if (UNLIKELY(UsePools && (Alignment > UE_MBC_STANDARD_ALIGNMENT)))
	{
		// check if allocations that require alignment larger than UE_MBC_STANDARD_ALIGNMENT can be promoted to a bin with a natural alignment that matches
		// i.e. 16 bytes allocation with 128 bytes alignment can be promoted to 128 bytes bin
		// this will save us a lot of memory as otherwise allocations will be promoted to OS allocs that are at least 4 KB large, depending on a platform
		UsePools = PromoteToLargerBin(Size, Alignment, *this);
	}

	if (UsePools)
	{
		const uint32 PoolIndex = BoundSizeToPoolIndex(Size, MemSizeToPoolIndex);
		FPerThreadFreeBlockLists* Lists = GMallocBinnedPerThreadCaches ? FPerThreadFreeBlockLists::Get() : nullptr;
		if (Lists)
		{
			if (Lists->ObtainRecycledPartial(PoolIndex, Private::GGlobalRecycler))
			{
				if (void* Result = Lists->Malloc(PoolIndex))
				{
#if UE_MB3_ALLOCATOR_STATS
					SmallPoolTables[PoolIndex].HeadEndAlloc(Size);
					const uint32 BinSize = SmallPoolTables[PoolIndex].BinSize;
					Lists->AllocatedMemory += BinSize;
#endif
					return Result;
				}
			}
		}

		NOALLOC_SCOPE_CYCLE_COUNTER(STAT_FMallocBinned3_MallocExternalSmall);

		// Allocate from small object pool.
		FPoolTable& Table = SmallPoolTables[PoolIndex];

		UE::TUniqueLock Lock(Table.Mutex);

		uint32 BlockIndex = MAX_uint32;
		FPoolInfoSmall* Pool = GetFrontPool(Table, PoolIndex, BlockIndex);
		if (!Pool)
		{
			Pool = PushNewPoolToFront(Table, Table.BinSize, PoolIndex, BlockIndex);

			// Indicates that we run out of reserved virtual memory for the pool (UE_MB3_MAX_MEMORY_PER_POOL_SIZE) for this bin type
			if (!Pool)
			{
				if ((PoolIndex + 1) < UE_MB3_SMALL_POOL_COUNT)
				{
					return FMallocBinned3::Malloc(SmallPoolTables[PoolIndex + 1].BinSize, Alignment);
				}
				else
				{
					return FMallocBinned3::Malloc(UE_MB3_MAX_SMALL_POOL_SIZE + 1, Alignment);
				}
			}
		}

		uint8* BlockPtr = BlockPointerFromIndecies(PoolIndex, BlockIndex, Table.BlockSize);

		void* Result = Pool->AllocateBin(BlockPtr, Table.BinSize);
#if UE_MB3_ALLOCATOR_STATS
		++Table.TotalUsedBins;
		Table.HeadEndAlloc(Size);
		AllocatedSmallPoolMemory.fetch_add(Table.BinSize, std::memory_order_relaxed);
#endif

#if UE_MBC_LIGHTWEIGHT_BIN_CALLSTACK_TRACKER
		TrackBinAllocation(Table.BinSize, Table.TotalAllocatedMem);
#endif

		if (GMallocBinnedAllocExtra)
		{
			if (Lists)
			{
				// prefill the free list with some allocations so we are less likely to hit this slow path with the mutex 
				for (int32 Index = 0; Index < GMallocBinnedAllocExtra && Pool->HasFreeBin(); Index++)
				{
					if (!Lists->Free(Result, PoolIndex, Table.BinSize))
					{
						break;
					}
					Result = Pool->AllocateBin(BlockPtr, Table.BinSize);
					UE_MBC_UPDATE_STATS(++Table.TotalUsedBins);
				}
			}
		}
		if (!Pool->HasFreeBin())
		{
			Table.BlocksExhaustedBits.AllocBit(BlockIndex);
		}

		return Result;
	}
	Alignment = FMath::Max<uint32>(Alignment, UE_MBC_STANDARD_ALIGNMENT);

	// Use OS for non-pooled allocations.
	const uint64 AlignedSize = Align(Size, FPlatformMemory::FPlatformVirtualMemoryBlock::GetCommitAlignment());

	LLM_PLATFORM_SCOPE(ELLMTag::FMalloc);

	NOALLOC_SCOPE_CYCLE_COUNTER(STAT_FMallocBinned3_MallocExternalLarge);

	LogLargeAllocation(AlignedSize);

#if UE_MB3_TIME_LARGE_BLOCKS
	const double StartTime = FPlatformTime::Seconds();
#endif

	void* Result;
	{
		size_t VMPages;

#if UE_MB3_USE_CACHED_PAGE_ALLOCATOR_FOR_LARGE_ALLOCS
		if (LargeAllocsCacheEnabledState.load(std::memory_order_relaxed) == EMB3CacheState::Enabled)
		{
			VMPages = AlignedSize / FPlatformMemory::FPlatformVirtualMemoryBlock::GetCommitAlignment();

			UE::TUniqueLock Lock(ExternalAllocMutex);
			Result = GetCachedOSPageAllocator().Allocate(AlignedSize);
		}
		else
#endif
		{
			FPlatformMemory::FPlatformVirtualMemoryBlock Block = FPlatformMemory::FPlatformVirtualMemoryBlock::AllocateVirtual(AlignedSize, Alignment);
			Block.Commit(0, AlignedSize);
			Result = Block.GetVirtualPointer();
			LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelAlloc(ELLMTracker::Platform, Result, AlignedSize));
			VMPages = Block.GetActualSizeInPages();
		}

		UE::TUniqueLock Lock(ExternalAllocMutex);
		if (!Result)
		{
			OutOfMemory(AlignedSize);
		}

		// Create pool.
		FPoolInfo* Pool = Internal::GetOrCreatePoolInfo(*this, Result, FPoolInfo::ECanary::Assigned);
		check(Size > 0 && Size <= AlignedSize && AlignedSize >= FPlatformMemory::FPlatformVirtualMemoryBlock::GetCommitAlignment());
		Pool->SetOSAllocationSizes(Size, AlignedSize, VMPages);
	}

#if UE_MB3_TIME_LARGE_BLOCKS
	const double Add = FPlatformTime::Seconds() - StartTime;
	double Old;
	do
	{
		Old = MemoryRangeReserveTotalTime.Load();
	} while (!MemoryRangeReserveTotalTime.CompareExchange(Old, Old + Add));
	MemoryRangeReserveTotalCount++;
#endif

	UE_CLOGF(!IsAligned(Result, Alignment) ,LogMemory, Fatal, "FMallocBinned3 alignment was too large for OS. Alignment=%d Ptr=%p", Alignment, Result);
	check(IsOSAllocation(Result));

#if UE_MB3_ALLOCATOR_STATS
	AllocatedLargePoolMemory.fetch_add(Size, std::memory_order_relaxed);
	AllocatedLargePoolMemoryWAlignment.fetch_add(AlignedSize, std::memory_order_relaxed);
#endif

	return Result;
}

void* FMallocBinned3::MallocNoTLS(SIZE_T Size, uint32 Alignment)
{
	bool UsePools = (Size <= UE_MB3_MAX_SMALL_POOL_SIZE);

	check(FMath::IsPowerOfTwo(Alignment));

	if (UNLIKELY(UsePools && (Alignment > UE_MBC_STANDARD_ALIGNMENT)))
	{
		// check if allocations that require alignment larger than UE_MBC_STANDARD_ALIGNMENT can be promoted to a bin with a natural alignment that matches
		// i.e. 16 bytes allocation with 128 bytes alignment can be promoted to 128 bytes bin
		// this will save us a lot of memory as otherwise allocations will be promoted to OS allocs that are at least 4 KB large, depending on a platform
		UsePools = PromoteToLargerBin(Size, Alignment, *this);
	}

	if (UsePools)
	{
		const uint32 PoolIndex = BoundSizeToPoolIndex(Size, MemSizeToPoolIndex);

		NOALLOC_SCOPE_CYCLE_COUNTER(STAT_FMallocBinned3_MallocExternalSmall);

		// Allocate from small object pool.
		FPoolTable& Table = SmallPoolTables[PoolIndex];

		UE::TUniqueLock Lock(Table.Mutex);

		uint32 BlockIndex = MAX_uint32;
		FPoolInfoSmall* Pool = GetFrontPool(Table, PoolIndex, BlockIndex);
		if (!Pool)
		{
			Pool = PushNewPoolToFront(Table, Table.BinSize, PoolIndex, BlockIndex);

			// Indicates that we run out of reserved virtual memory for the pool (UE_MB3_MAX_MEMORY_PER_POOL_SIZE) for this bin type
			if (!Pool)
			{
				if ((PoolIndex + 1) < UE_MB3_SMALL_POOL_COUNT)
				{
					return FMallocBinned3::Malloc(SmallPoolTables[PoolIndex + 1].BinSize, Alignment);
				}
				else
				{
					return FMallocBinned3::Malloc(UE_MB3_MAX_SMALL_POOL_SIZE + 1, Alignment);
				}
			}
		}

		uint8* BlockPtr = BlockPointerFromIndecies(PoolIndex, BlockIndex, Table.BlockSize);

		void* Result = Pool->AllocateBin(BlockPtr, Table.BinSize);
#if UE_MB3_ALLOCATOR_STATS
		++Table.TotalUsedBins;
		Table.HeadEndAlloc(Size);
		AllocatedSmallPoolMemory.fetch_add(Table.BinSize, std::memory_order_relaxed);
#endif

#if UE_MBC_LIGHTWEIGHT_BIN_CALLSTACK_TRACKER
		TrackBinAllocation(Table.BinSize, Table.TotalAllocatedMem);
#endif

		if (!Pool->HasFreeBin())
		{
			Table.BlocksExhaustedBits.AllocBit(BlockIndex);
		}

		return Result;
	}

	return Malloc(Size, Alignment);
}

void* FMallocBinned3::Realloc(void* Ptr, SIZE_T NewSize, uint32 Alignment)
{
	const uint64 PoolIndex = PoolIndexFromPtr(Ptr);
	if (NewSize == 0)
	{
		FMallocBinned3::FreeExternal(Ptr, PoolIndex);
		return nullptr;
	}

	check(FMath::IsPowerOfTwo(Alignment));
	check(Alignment <= OsAllocationGranularity);

	if (PoolIndex < UE_MB3_SMALL_POOL_COUNT)
	{
		check(Ptr); // null is an OS allocation because it will not fall in our VM block
		const uint32 BinSize = PoolIndexToBinSize(PoolIndex);
		if (
			((NewSize <= BinSize) & (IsAligned(BinSize, Alignment))) && // one branch, not two
			(PoolIndex == 0 || NewSize > PoolIndexToBinSize(PoolIndex - 1)))
		{
#if UE_MB3_ALLOCATOR_STATS
			SmallPoolTables[PoolIndex].HeadEndAlloc(NewSize);
			SmallPoolTables[PoolIndex].HeadEndFree();
#endif
			return Ptr;
		}

		// Reallocate and copy the data across
		void* Result = FMallocBinned3::Malloc(NewSize, Alignment);
		FMemory::Memcpy(Result, Ptr, FMath::Min<SIZE_T>(NewSize, BinSize));
		FMallocBinned3::FreeExternal(Ptr, PoolIndex);
		return Result;
	}
	if (!Ptr)
	{
		void* Result = FMallocBinned3::Malloc(NewSize, Alignment);
		return Result;
	}

	NOALLOC_SCOPE_CYCLE_COUNTER(STAT_FMallocBinned3_ReallocExternal);

	ExternalAllocMutex.Lock();

	// Allocated from OS.
	FPoolInfo* Pool = Internal::FindPoolInfo(*this, Ptr);
	if (!Pool)
	{
		UE_LOGF(LogMemory, Fatal, "FMallocBinned3 Attempt to realloc an unrecognized pointer %p", Ptr);
	}
	const uint32 PoolOsBytes = Pool->GetOsCommittedBytes();
	const uint32 PoolOSRequestedBytes = Pool->GetOSRequestedBytes();
	checkf(PoolOSRequestedBytes <= PoolOsBytes, TEXT("FMallocBinned3::ReallocExternal %u %u"), PoolOSRequestedBytes, PoolOsBytes);
	if (NewSize > PoolOsBytes || // can't fit in the old block
		(NewSize <= UE_MB3_MAX_SMALL_POOL_SIZE && Alignment <= UE_MBC_STANDARD_ALIGNMENT) || // can switch to the small bin allocator
		Align(NewSize, OsAllocationGranularity) < PoolOsBytes) // we can get some pages back
	{
		ExternalAllocMutex.Unlock();
		// Grow or shrink.
		void* Result = FMallocBinned3::Malloc(NewSize, Alignment);
		SIZE_T CopySize = FMath::Min<SIZE_T>(NewSize, PoolOSRequestedBytes);
		FMemory::Memcpy(Result, Ptr, CopySize);
		FMallocBinned3::FreeExternal(Ptr, PoolIndex);
		return Result;
	}

	UE_MBC_UPDATE_STATS(AllocatedLargePoolMemory.fetch_add((int64)NewSize - (int64)PoolOSRequestedBytes, std::memory_order_relaxed));
	// don't need to change the Binned3AllocatedLargePoolMemoryWAlignment because we didn't reallocate so it's the same size
	
	Pool->SetOSAllocationSize(NewSize);
	ExternalAllocMutex.Unlock();
	return Ptr;
}

void FMallocBinned3::FreeExternal(void* Ptr, uint64 PoolIndex)
{
	NOALLOC_SCOPE_CYCLE_COUNTER(STAT_FMallocBinned3_FreeExternal);

	if (PoolIndex < UE_MB3_SMALL_POOL_COUNT)
	{
		check(Ptr); // null is an OS allocation because it will not fall in our VM block
		const uint32 BinSize = PoolIndexToBinSize(PoolIndex);

		FBundleNode* BundlesToRecycle = nullptr;
		FPerThreadFreeBlockLists* Lists = GMallocBinnedPerThreadCaches ? FPerThreadFreeBlockLists::Get() : nullptr;
		if (Lists)
		{
			BundlesToRecycle = Lists->RecycleFullBundle(PoolIndex, Private::GGlobalRecycler);
			const bool bPushed = Lists->Free(Ptr, PoolIndex, BinSize);
			check(bPushed);
#if UE_MB3_ALLOCATOR_STATS
			SmallPoolTables[PoolIndex].HeadEndFree();
			Lists->AllocatedMemory -= BinSize;
#endif
		}
		else
		{
			BundlesToRecycle = (FBundleNode*)Ptr;
			BundlesToRecycle->SetNextNodeInCurrentBundle(nullptr);
		}
		if (BundlesToRecycle)
		{
			Private::FreeBundles(*this, BundlesToRecycle, PoolIndex);
#if UE_MB3_ALLOCATOR_STATS
			if (!Lists)
			{
				SmallPoolTables[PoolIndex].HeadEndFree();
				// lists track their own stat track them instead in the global stat if we don't have lists
				AllocatedSmallPoolMemory.fetch_sub((int64)BinSize, std::memory_order_relaxed);
			}
#endif
		}
	}
	else if (Ptr)
	{
		FPoolInfo* Pool;
		{
			UE::TUniqueLock Lock(ExternalAllocMutex);

			Pool = Internal::FindPoolInfo(*this, Ptr);
		}

		if (!Pool)
		{
			UE_LOGF(LogMemory, Fatal, "FMallocBinned3 Attempt to free an unrecognized pointer %p", Ptr);
		}
		const uint32 PoolOsBytes = Pool->GetOsCommittedBytes();
		const uint32 PoolOSRequestedBytes = Pool->GetOSRequestedBytes();
		const uint32 VMPages = Pool->GetOsVMPages();

#if UE_MB3_ALLOCATOR_STATS
		AllocatedLargePoolMemory.fetch_sub((int64)PoolOSRequestedBytes, std::memory_order_relaxed);
		AllocatedLargePoolMemoryWAlignment.fetch_sub((int64)PoolOsBytes, std::memory_order_relaxed);
#endif

		checkf(PoolOSRequestedBytes <= PoolOsBytes, TEXT("FMallocBinned3::FreeExternal %u %u"), int32(PoolOSRequestedBytes), int32(PoolOsBytes));
		Pool->SetCanary(FPoolInfo::ECanary::Unassigned, true, false);

		// Free an OS allocation.
#if UE_MB3_TIME_LARGE_BLOCKS
		const double StartTime = FPlatformTime::Seconds();
#endif

#if UE_MB3_USE_CACHED_PAGE_ALLOCATOR_FOR_LARGE_ALLOCS
		if (LargeAllocsCacheEnabledState.load(std::memory_order_relaxed) == EMB3CacheState::Enabled)
		{
			UE::TUniqueLock Lock(ExternalAllocMutex);
			GetCachedOSPageAllocator().Free(Ptr, VMPages * FPlatformMemory::FPlatformVirtualMemoryBlock::GetCommitAlignment());
		}
		else
#endif
		{
			LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelFree(ELLMTracker::Platform, Ptr));
			FPlatformMemory::FPlatformVirtualMemoryBlock Block(Ptr, VMPages);
			Block.FreeVirtual();
		}

#if UE_MB3_TIME_LARGE_BLOCKS
		const double Add = FPlatformTime::Seconds() - StartTime;
		double Old;
		do
		{
			Old = MemoryRangeFreeTotalTime.Load();
		} while (!MemoryRangeFreeTotalTime.CompareExchange(Old, Old + Add));
		MemoryRangeFreeTotalCount++;
#endif
	}
}

void FMallocBinned3::FreeNoTLS(void* Ptr)
{
	NOALLOC_SCOPE_CYCLE_COUNTER(STAT_FMallocBinned3_FreeExternal);

	const uint64 PoolIndex = PoolIndexFromPtr(Ptr);
	if (PoolIndex < UE_MB3_SMALL_POOL_COUNT)
	{
		check(Ptr); // null is an OS allocation because it will not fall in our VM block
		const uint32 BinSize = PoolIndexToBinSize(PoolIndex);

		FBundleNode* BundlesToRecycle = (FBundleNode*)Ptr;
		BundlesToRecycle->SetNextNodeInCurrentBundle(nullptr);

		Private::FreeBundles(*this, BundlesToRecycle, PoolIndex);
#if UE_MB3_ALLOCATOR_STATS
		SmallPoolTables[PoolIndex].HeadEndFree();
		// lists track their own stat track them instead in the global stat if we don't have lists
		AllocatedSmallPoolMemory.fetch_sub((int64)BinSize, std::memory_order_relaxed);
#endif
	}
	else if (Ptr)
	{
		FreeExternal(Ptr, PoolIndex);
	}
}

bool FMallocBinned3::ValidateHeap()
{
	// Not implemented
	// NumEverUsedBlocks gives us all of the information we need to examine each pool, so it is doable.
	return true;
}

const TCHAR* FMallocBinned3::GetDescriptiveName()
{
	return TEXT("Binned3");
}

void FMallocBinned3::Trim(bool bTrimThreadCaches)
{
	if (GMallocBinnedPerThreadCaches && bTrimThreadCaches)
	{
		// Trim memory and increase the Epoch.
		FMallocBinnedCommonUtils::Trim(*this);
	}

#if UE_MB3_USE_CACHED_PAGE_ALLOCATOR_FOR_LARGE_ALLOCS
	if (LargeAllocsCacheEnabledState.load(std::memory_order_relaxed) == EMB3CacheState::Disabled)
	{
		// trim cache as some allocations might have been stored in it due to a race condition
		UE::TUniqueLock Lock(ExternalAllocMutex);
		GetCachedOSPageAllocator().FreeAll();
		LargeAllocsCacheEnabledState.store(EMB3CacheState::Trimmed, std::memory_order_relaxed);
	}
#endif
}

void FMallocBinned3::FlushCurrentThreadCacheInternal(bool bNewEpochOnly)
{
	FMallocBinnedCommonUtils::FlushCurrentThreadCache(*this, bNewEpochOnly);
}

void FMallocBinned3::EnableLargeAllocsCache()
{
#if UE_MB3_USE_CACHED_PAGE_ALLOCATOR_FOR_LARGE_ALLOCS
	LargeAllocsCacheEnabledState.store(EMB3CacheState::Enabled, std::memory_order_relaxed);
#endif
}

void FMallocBinned3::DisableLargeAllocsCache()
{
#if UE_MB3_USE_CACHED_PAGE_ALLOCATOR_FOR_LARGE_ALLOCS
	UE::TUniqueLock Lock(ExternalAllocMutex);
	LargeAllocsCacheEnabledState.store(EMB3CacheState::Disabled, std::memory_order_relaxed);
	GetCachedOSPageAllocator().FreeAll();
#endif
}

void FMallocBinned3::DumpAllocatorStats(class FOutputDevice& Ar)
{
#if UE_MB3_ALLOCATOR_STATS
	const int64 TotalAllocatedSmallPoolMemory = GetTotalAllocatedSmallPoolMemory();

	Ar.Logf(TEXT("FMallocBinned3 Mem report"));
	Ar.Logf(TEXT("Constants.BinnedAllocationGranularity = %d"), int32(OsAllocationGranularity));
	Ar.Logf(TEXT("UE_MB3_MAX_SMALL_POOL_SIZE = %d"), int32(UE_MB3_MAX_SMALL_POOL_SIZE));
	Ar.Logf(TEXT("UE_MB3_MAX_MEMORY_PER_POOL_SIZE = %llu"), uint64(UE_MB3_MAX_MEMORY_PER_POOL_SIZE));
	Ar.Logf(TEXT("Small Pool Allocations: %fmb  (including bin size padding)"), ((double)TotalAllocatedSmallPoolMemory) / (1024.0f * 1024.0f));
	Ar.Logf(TEXT("Small Pool OS Allocated: %fmb"), ((double)AllocatedOSSmallPoolMemory.load(std::memory_order_relaxed)) / (1024.0f * 1024.0f));
	Ar.Logf(TEXT("Large Pool Requested Allocations: %fmb"), ((double)AllocatedLargePoolMemory.load(std::memory_order_relaxed)) / (1024.0f * 1024.0f));
	Ar.Logf(TEXT("Large Pool OS Allocated: %fmb"), ((double)AllocatedLargePoolMemoryWAlignment.load(std::memory_order_relaxed)) / (1024.0f * 1024.0f));
	Ar.Logf(TEXT("PoolInfo: %fmb"), ((double)PoolInfoMemory) / (1024.0f * 1024.0f));
	Ar.Logf(TEXT("Hash: %fmb"), ((double)HashMemory) / (1024.0f * 1024.0f));
	Ar.Logf(TEXT("Free Bits: %fmb"), ((double)Binned3FreeBitsMemory) / (1024.0f * 1024.0f));
	Ar.Logf(TEXT("TLS: %fmb"), ((double)TLSMemory.load(std::memory_order_relaxed)) / (1024.0f * 1024.0f));
	Ar.Logf(TEXT("Slab Commits: %llu"), Binned3Commits.Load());
	Ar.Logf(TEXT("Slab Decommits: %llu"), Binned3Decommits.Load());
	Ar.Logf(TEXT("Total allocated from OS: %fmb"), 
		((double)
			AllocatedOSSmallPoolMemory.load(std::memory_order_relaxed) + AllocatedLargePoolMemoryWAlignment.load(std::memory_order_relaxed) + PoolInfoMemory + HashMemory + Binned3FreeBitsMemory + TLSMemory.load(std::memory_order_relaxed)
			) / (1024.0f * 1024.0f));


#if UE_MB3_TIME_LARGE_BLOCKS
	Ar.Logf(TEXT("MemoryRangeReserve %d calls %6.3fs    %6.3fus / call"), MemoryRangeReserveTotalCount.Load(), float(MemoryRangeReserveTotalTime.Load()), float(MemoryRangeReserveTotalTime.Load()) * 1000000.0f / float(MemoryRangeReserveTotalCount.Load()));
	Ar.Logf(TEXT("MemoryRangeFree    %d calls %6.3fs    %6.3fus / call"), MemoryRangeFreeTotalCount.Load(), float(MemoryRangeFreeTotalTime.Load()), float(MemoryRangeFreeTotalTime.Load()) * 1000000.0f / float(MemoryRangeFreeTotalCount.Load()));
#endif

	for (int32 i = 0; i < UE_MB3_SMALL_POOL_COUNT; i++)
	{
		const float Fragmentation = (SmallPoolTables[i].TotalAllocatedBins > 0) ? 1.0f - (float)SmallPoolTables[i].TotalUsedBins / (float)SmallPoolTables[i].TotalAllocatedBins : 0;
		const float TotalMem = (float)SmallPoolTables[i].TotalAllocatedMem / 1024.0f / 1024.0f;
		Ar.Logf(TEXT("Bin %6d Fragmentation %d %%, Wasted Mem %.2f MB, Total Allocated Mem %.2f MB"), 
			PoolIndexToBinSize(i), int(Fragmentation * 100.0f), TotalMem * Fragmentation, TotalMem);
	}

#if UE_M3_ALLOCATOR_PER_BIN_STATS
	for (int32 PoolIndex = 0; PoolIndex < UE_MB3_SMALL_POOL_COUNT; PoolIndex++)
	{
		const int64 VM = SmallPoolTables[PoolIndex].UnusedAreaOffsetLow;
		const uint32 CommittedBlocks = SmallPoolTables[PoolIndex].BlocksAllocatedBits.CountOnes(SmallPoolTables[PoolIndex].NumEverUsedBlocks);
		const uint32 PartialBlocks = SmallPoolTables[PoolIndex].NumEverUsedBlocks - SmallPoolTables[PoolIndex].BlocksExhaustedBits.CountOnes(SmallPoolTables[PoolIndex].NumEverUsedBlocks);
		const uint32 FullBlocks = CommittedBlocks - PartialBlocks;
		const int64 ComittedVM = VM - (SmallPoolTables[PoolIndex].NumEverUsedBlocks - CommittedBlocks) * SmallPoolTables[PoolIndex].BlockSize;

		const int64 AveSize = SmallPoolTables[PoolIndex].TotalAllocCount.load(std::memory_order_relaxed) ? SmallPoolTables[PoolIndex].TotalRequestedAllocSize.load(std::memory_order_relaxed) / SmallPoolTables[PoolIndex].TotalAllocCount.load(std::memory_order_relaxed) : 0;
		const int64 EstPadWaste = ((SmallPoolTables[PoolIndex].TotalAllocCount.load(std::memory_order_relaxed) - SmallPoolTables[PoolIndex].TotalFreeCount.load(std::memory_order_relaxed)) * (PoolIndexToBinSize(PoolIndex) - AveSize));

		Ar.Logf(TEXT("Pool %2d   Size %6d   Allocs %8lld  Frees %8lld  AveAllocSize %6d  EstPadWaste %4dKB  UsedVM %3dMB  CommittedVM %3dMB  HighSlabs %6d  CommittedSlabs %6d  FullSlabs %6d  PartialSlabs  %6d"), 
			PoolIndex,
			PoolIndexToBinSize(PoolIndex),
			SmallPoolTables[PoolIndex].TotalAllocCount.load(std::memory_order_relaxed),
			SmallPoolTables[PoolIndex].TotalFreeCount.load(std::memory_order_relaxed),
			AveSize,
			EstPadWaste / 1024,
			VM / (1024 * 1024),
			ComittedVM / (1024 * 1024),
			SmallPoolTables[PoolIndex].NumEverUsedBlocks,
			CommittedBlocks,
			FullBlocks,
			PartialBlocks
			);
	}
#endif

#else
	Ar.Logf(TEXT("Allocator Stats for Binned3 are not in this build set UE_MB3_ALLOCATOR_STATS 1 in MallocBinned3.cpp"));
#endif
}

void FMallocBinned3::UpdateStats()
{
	FMallocBinnedCommonUtils::UpdateStatsCommon(*this);
}

#endif //~PLATFORM_HAS_FPlatformVirtualMemoryBlock

PRAGMA_RESTORE_UNSAFE_TYPECAST_WARNINGS
