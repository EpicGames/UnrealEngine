// Copyright Epic Games, Inc. All Rights Reserved.

#include "HAL/MallocBinned4.h"
#include "Templates/Function.h"
#include "GenericPlatform/GenericPlatformProcess.h"
#include "Stats/Stats.h"
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformMisc.h"
#include "Misc/App.h"
#include "Misc/CoreDelegates.h"
#include "HAL/MallocTimer.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "HAL/MallocBinnedCommonUtils.h"

CSV_DECLARE_CATEGORY_MODULE_EXTERN(CORE_API, FMemory);

PRAGMA_DISABLE_UNSAFE_TYPECAST_WARNINGS


#define UE_DEFAULT_GBinned4MoveOSFreesOffTimeCriticalThreads 1

#if UE_MBC_ALLOW_RUNTIME_TWEAKING
	int32 GBinned4MoveOSFreesOffTimeCriticalThreads = UE_DEFAULT_GBinned4MoveOSFreesOffTimeCriticalThreads;
	static FAutoConsoleVariableRef GGMallocBinned4MoveOSFreesOffTimeCriticalThreadsCVar(
		TEXT("MallocBinned4.MoveOSFreesOffTimeCriticalThreads"),
		GBinned4MoveOSFreesOffTimeCriticalThreads,
		TEXT("When the OS needs to free memory hint to the underlying cache that we are on a time critical thread, it may decide to delay the free for a non time critical thread")
	);
#else
#	define GBinned4MoveOSFreesOffTimeCriticalThreads	UE_DEFAULT_GBinned4MoveOSFreesOffTimeCriticalThreads
#endif

#if UE_MB4_ALLOCATOR_STATS
	int64 Binned4HashMemory = 0;
#endif

// Bin sizes are based around getting the maximum amount of allocations per pool, with as little alignment waste as possible.
// Bin sizes should be close to even divisors of the system page size, and well distributed.
// All bins, but 8, must be 16-byte aligned.
constexpr uint16 MB4SmallBinSizesInternal[] =
{
	8,
	16, 32, 48, 64, 80, 96, 112, 128, 144, 160, 176, 192, 208, // +16
//	Bin		|	Divider | Slack leftover per page (64KB)
	256,		// /256
	288,		// /227 160b
	320,		// /204 256b
	384,		// /170 256b
	448,		// /146 128b
	512,		// /128	
	560,		// /117	16b
	624,		// /105	16b
	720,		// /91	16b
	816,		// /80	256b
	912,		// /71	784b
	1024,		// /64
	1168,		// /56	128b
	1392,		// /47	112b
	1520,		// /43	176b
	1680,		// /39	16b
	1872,		// /35	16b
	2048,		// /32
	2256,		// /29	112b
	2608,		// /25	336b
	2976,		// /22	64b
	3264,		// /20	256b
	3632,		// /18	160b
	4096,		// /16
	4368,		// /15	16b
	4672,		// /14	128b
	5040,		// /13	16b
	5456,		// /12	64b
	5952,		// /11	64b
	6544,		// /10	96b
	7280,		// /9	16b
	8192,		// /8
	9360,		// /7	16b
	10912,		// /6	64b
	13104,		// /5	16b
#if !AGGRESSIVE_MEMORY_SAVING
	16384,		// /4
	21840,		// /3	16b
	32768		// /2
#endif
};

static uint32 GMB4PageSize = 0;

uint16 FMallocBinned4::SmallBinSizes[UE_MB4_SMALL_POOL_COUNT] = { 0 };
FMallocBinned4* FMallocBinned4::MallocBinned4 = nullptr;
// Mapping of sizes to small table indices
uint8 FMallocBinned4::MemSizeToPoolIndex[SIZE_TO_POOL_INDEX_NUM] = { 0 };

template <>
thread_local FMallocBinned4::FPerThreadFreeBlockLists* FMallocBinned4::FPerThreadFreeBlockLists::ThreadSingleton = nullptr;


static FORCEINLINE bool MB4IsSupportedSize(SIZE_T Size)
{
	bool bResult = IsAligned(Size, UE_MBC_MIN_SMALL_POOL_ALIGNMENT);
	bResult = bResult && (Size >> UE_MBC_BIN_SIZE_SHIFT) <= SIZE_T(MAX_uint32);
	return bResult;
}

FMallocBinned4::FPoolInfo::FPoolInfo()
	: Taken(0)
	, Canary(ECanary::Unassigned)
	, AllocSize(0)
	, FirstFreeBlock(nullptr)
	, Next(nullptr)
	, PtrToPrevNext(nullptr)
{
}

void FMallocBinned4::FPoolInfo::CheckCanary(ECanary ShouldBe) const
{
	if (Canary != ShouldBe)
	{
		UE_LOGF(LogMemory, Fatal, "MallocBinned4 Corruption Canary was 0x%x, should be 0x%x", int32(Canary), int32(ShouldBe));
	}
}

void FMallocBinned4::FPoolInfo::SetCanary(ECanary ShouldBe, bool bPreexisting, bool bGuaranteedToBeNew)
{
	if (bPreexisting)
	{
		if (bGuaranteedToBeNew)
		{
			UE_LOGF(LogMemory, Fatal, "MallocBinned4 Corruption Canary was 0x%x, should be 0x%x. This block is both preexisting and guaranteed to be new; which makes no sense.", int32(Canary), int32(ShouldBe));
		}
		if (ShouldBe == ECanary::Unassigned)
		{
			if (Canary != ECanary::FirstFreeBlockIsOSAllocSize && Canary != ECanary::FirstFreeBlockIsPtr)
			{
				UE_LOGF(LogMemory, Fatal, "MallocBinned4 Corruption Canary was 0x%x, will be 0x%x because this block should be preexisting and in use.", int32(Canary), int32(ShouldBe));
			}
		}
		else if (Canary != ShouldBe)
		{
			UE_LOGF(LogMemory, Fatal, "MallocBinned4 Corruption Canary was 0x%x, should be 0x%x because this block should be preexisting.", int32(Canary), int32(ShouldBe));
		}
	}
	else
	{
		if (bGuaranteedToBeNew)
		{
			if (Canary != ECanary::Unassigned)
			{
				UE_LOGF(LogMemory, Fatal, "MallocBinned4 Corruption Canary was 0x%x, will be 0x%x. This block is guaranteed to be new yet is it already assigned.", int32(Canary), int32(ShouldBe));
			}
		}
		else if (Canary != ShouldBe && Canary != ECanary::Unassigned)
		{
			UE_LOGF(LogMemory, Fatal, "MallocBinned4 Corruption Canary was 0x%x, will be 0x%x does not have an expected value.", int32(Canary), int32(ShouldBe));
		}
	}
	Canary = ShouldBe;
}

bool FMallocBinned4::FPoolInfo::HasFreeBin() const
{
	CheckCanary(ECanary::FirstFreeBlockIsPtr);
	return FirstFreeBlock && FirstFreeBlock->GetNumFreeBins() != 0;
}

void* FMallocBinned4::FPoolInfo::AllocateBin()
{
	check(HasFreeBin());
	++Taken;
	void* Result = FirstFreeBlock->AllocateBin();
	ExhaustPoolIfNecessary();
	return Result;
}

SIZE_T FMallocBinned4::FPoolInfo::GetOSRequestedBytes() const
{
	return SIZE_T(AllocSize) << UE_MBC_BIN_SIZE_SHIFT;
}

SIZE_T FMallocBinned4::FPoolInfo::GetOsAllocatedBytes() const
{
	CheckCanary(ECanary::FirstFreeBlockIsOSAllocSize);
	return (SIZE_T)FirstFreeBlock;
}

void FMallocBinned4::FPoolInfo::SetOSAllocationSizes(SIZE_T InRequestedBytes, UPTRINT InAllocatedBytes)
{
	CheckCanary(ECanary::FirstFreeBlockIsOSAllocSize);
	checkSlow(InRequestedBytes != 0);                // Shouldn't be pooling zero byte allocations
	checkSlow(InAllocatedBytes >= InRequestedBytes); // We must be allocating at least as much as we requested
	checkSlow(MB4IsSupportedSize(InRequestedBytes));    // We must be allocating a size we can store

	AllocSize      = uint32(InRequestedBytes >> UE_MBC_BIN_SIZE_SHIFT);
	FirstFreeBlock = (FFreeBlock*)InAllocatedBytes;
}

void FMallocBinned4::FPoolInfo::Link(FPoolInfo*& PrevNext)
{
	if (PrevNext)
	{
		PrevNext->PtrToPrevNext = &Next;
	}
	Next          = PrevNext;
	PtrToPrevNext = &PrevNext;
	PrevNext      = this;
}

void FMallocBinned4::FPoolInfo::Unlink()
{
	if (Next)
	{
		Next->PtrToPrevNext = PtrToPrevNext;
	}
	*PtrToPrevNext = Next;
}

void FMallocBinned4::FPoolInfo::ExhaustPoolIfNecessary()
{
	if (FirstFreeBlock->GetNumFreeBins() == 0)
	{
		FirstFreeBlock = FirstFreeBlock->GetNextFreeBlock();
	}
	check(!FirstFreeBlock || FirstFreeBlock->GetNumFreeBins() != 0);
}

struct FMallocBinned4::Private
{
	static MallocBinnedPrivate::TGlobalRecycler<UE_MB4_SMALL_POOL_COUNT> GGlobalRecycler;

	static void FreeBundles(FMallocBinned4& Allocator, FBundleNode* BundlesToRecycle, uint32 InPoolIndex)
	{
		FPoolTable& Table = Allocator.SmallPoolTables[InPoolIndex];
		const uint32 BinSize = Table.BinSize;

		UE::TUniqueLock Lock(Table.Mutex);

		FBundleNode* Node = BundlesToRecycle;
		do
		{
			FBundleNode* NextNode = Node->GetNextNodeInCurrentBundle();
			FPoolInfo*   NodePool = Internal::FindPoolInfo(Allocator, Node);
			if (!NodePool)
			{
				UE_LOGF(LogMemory, Fatal, "FMallocBinned4 Attempt to free an unrecognized small block %p", Node);
			}
			NodePool->CheckCanary(FPoolInfo::ECanary::FirstFreeBlockIsPtr);

			// If this pool was exhausted, move to available list.
			if (!NodePool->FirstFreeBlock)
			{
				Table.ActivePools.LinkToFront(NodePool);
			}
			else
			{
				// If we are freeing memory in this pool it must have the current canary and not the pre-fork one. All caches should have been cleared when forking.
				check(NodePool->FirstFreeBlock->CanaryAndForkState == EBlockCanary::Zero || NodePool->FirstFreeBlock->CanaryAndForkState == Allocator.CurrentCanary);
			}

			// Free a pooled allocation.
			FFreeBlock* Free = (FFreeBlock*)Node;
			Free->NumFreeBins = 1;
			Free->SetNextFreeBlock(NodePool->FirstFreeBlock);
			Free->BinSize = BinSize;
			Free->CanaryAndForkState = Allocator.CurrentCanary;
			NodePool->FirstFreeBlock = Free;

			UE_MBC_UPDATE_STATS(--Table.TotalUsedBins);

			// Free this pool.
			check(NodePool->Taken >= 1);
			if (--NodePool->Taken == 0)
			{
				NodePool->SetCanary(FPoolInfo::ECanary::Unassigned, true, false);
				void* BasePtrOfNode = (void*)AlignDown(Node, UE_MB4_LARGE_ALLOC);

				// Free the OS memory.
				NodePool->Unlink();
				{
					UE::TUniqueLock InnerLock(Allocator.ExternalAllocMutex);
					Allocator.UnregisterBlockPool(BasePtrOfNode);
					Allocator.CachedOSPageAllocator.Free(BasePtrOfNode, GMB4PageSize);
				}

#if UE_MB4_ALLOCATOR_STATS
				const int NumBinsInBlock = GMB4PageSize / BinSize;
				Table.TotalAllocatedBins -= NumBinsInBlock;
				Table.TotalAllocatedMem -= GMB4PageSize;
				AllocatedOSSmallPoolMemory.fetch_sub((int64)GMB4PageSize, std::memory_order_relaxed);
#endif
			}

			Node = NextNode;
		} while (Node);
	}

#if BINNED4_FORK_SUPPORT
	static void CheckThreadFreeBlockListsForFork()
	{
		if(GMallocBinnedPerThreadCaches)
		{
			UE::TUniqueLock Lock(GetFreeBlockListsRegistrationMutex());
			TArray<FPerThreadFreeBlockLists*>& List = GetRegisteredFreeBlockLists();
			UE_CLOGF(List.Num() == 1 && List[0] != FPerThreadFreeBlockLists::Get(), LogMemory, Fatal, "There was a thread-local free list at fork time which did not belong to the main forking thread. No other threads should be alive at fork time. If threads are spawned before forking, they must be killed and FMallocBinned4::ClearAndDisableTLSCachesOnCurrentThread() must be called.");
			UE_CLOGF(List.Num() > 1, LogMemory, Fatal, "There were multiple thread-local free lists at fork time. No other threads should be alive at fork time. If threads are spawned before forking, they must be killed and FMallocBinned4::ClearAndDisableTLSCachesOnCurrentThread() must be called.");
		}
	}
#endif
};

MallocBinnedPrivate::TGlobalRecycler<UE_MB4_SMALL_POOL_COUNT> FMallocBinned4::Private::GGlobalRecycler;

FORCEINLINE void FMallocBinned4::FPoolList::Clear()
{
	Front = nullptr;
}

FORCEINLINE bool FMallocBinned4::FPoolList::IsEmpty() const
{
	return Front == nullptr;
}

FORCEINLINE FMallocBinned4::FPoolInfo& FMallocBinned4::FPoolList::GetFrontPool()
{
	check(!IsEmpty());
	return *Front;
}

FORCEINLINE const FMallocBinned4::FPoolInfo& FMallocBinned4::FPoolList::GetFrontPool() const
{
	check(!IsEmpty());
	return *Front;
}

void FMallocBinned4::FPoolList::LinkToFront(FPoolInfo* Pool)
{
	Pool->Unlink();
	Pool->Link(Front);
}

FMallocBinned4::FPoolInfo& FMallocBinned4::FPoolList::PushNewPoolToFront(FMallocBinned4& Allocator, FPoolTable& Table, uint32 InPoolIndex)
{
	LLM_PLATFORM_SCOPE(ELLMTag::FMalloc);

	void* FreePtr;
	{
		UE::TUniqueLock Lock(Allocator.ExternalAllocMutex);
		FreePtr = Allocator.CachedOSPageAllocator.Allocate(GMB4PageSize, FMemory::AllocationHints::SmallPool);
	}
	if (!FreePtr)
	{
		OutOfMemory(GMB4PageSize);
	}

	FFreeBlock* Free = new (FreePtr) FFreeBlock(GMB4PageSize, Table.BinSize, Allocator.CurrentCanary);
	check(IsAligned(Free, GMB4PageSize));

#if UE_MB4_ALLOCATOR_STATS
	Table.TotalAllocatedBins += Free->NumFreeBins;
	Table.TotalAllocatedMem += GMB4PageSize;
	AllocatedOSSmallPoolMemory.fetch_add((int64)GMB4PageSize, std::memory_order_relaxed);
#endif

	UE::TUniqueLock Lock(Allocator.ExternalAllocMutex);
	Allocator.RegisterBlockPool(FreePtr, InPoolIndex);
	// Create pool
	FPoolInfo* Result = Internal::GetOrCreatePoolInfo(Allocator, Free, FPoolInfo::ECanary::FirstFreeBlockIsPtr);
	Result->Link(Front);
	Result->Taken          = 0;
	Result->FirstFreeBlock = Free;

	return *Result;
}

FMallocBinned4::FMallocBinned4()
{
	static bool bOnce = false;
	check(!bOnce); // this is now a singleton-like thing and you cannot make multiple copies
	bOnce = true;

	FMemory::Memcpy(SmallBinSizes, MB4SmallBinSizesInternal);
	
	static_assert(sizeof(FFreeBlock)  <= MB4SmallBinSizesInternal[0], "sizeof(FFreeBlock)  must be fit in smallest allocation size handled by FMallocBinned4.");
	static_assert(sizeof(FBundleNode) <= MB4SmallBinSizesInternal[0], "sizeof(FBundleNode) must fit in smallest allocation size handled by FMallocBinned4.");

	const FGenericPlatformMemoryConstants Constants = FPlatformMemory::GetConstants();
	GMB4PageSize = Constants.BinnedPageSize;
	OsAllocationGranularity = Constants.BinnedAllocationGranularity ? Constants.BinnedAllocationGranularity : GMB4PageSize;
	NumPoolsPerPage = GMB4PageSize / sizeof(FPoolInfo);
	PtrToPoolMapping.Init(GMB4PageSize, NumPoolsPerPage, Constants.AddressStart, Constants.AddressLimit);

	checkf(FMath::IsPowerOfTwo(GMB4PageSize), TEXT("OS page size must be a power of two"));
	checkf(Constants.AddressLimit > GMB4PageSize, TEXT("OS address limit must be greater than the page size")); // Check to catch 32 bit overflow in AddressLimit
	checkf(GMB4PageSize % UE_MB4_LARGE_ALLOC == 0, TEXT("OS page size must be a multiple of UE_MB4_LARGE_ALLOC"));

	static_assert(MB4SmallBinSizesInternal[UE_MB4_SMALL_POOL_COUNT - 1] == UE_MB4_MAX_SMALL_POOL_SIZE, "UE_MB4_MAX_SMALL_POOL_SIZE must equal the largest bin size");
	static_assert(UE_ARRAY_COUNT(MB4SmallBinSizesInternal) == UE_MB4_SMALL_POOL_COUNT, "Number of bins in MB4SmallBinSizesInternal must match UE_MB4_SMALL_POOL_COUNT");
	static_assert(UE_ARRAY_COUNT(MB4SmallBinSizesInternal) <= 256, "Number of bins in MB4SmallBinSizesInternal fit in one byte");

	// Init pool tables.
	for (uint32 Index = 0; Index != UE_MB4_SMALL_POOL_COUNT; ++Index)
	{
		checkf(Index == 0 || SmallBinSizes[Index - 1] < SmallBinSizes[Index], TEXT("Small bin sizes must be strictly increasing"));
		checkf(SmallBinSizes[Index] <= GMB4PageSize, TEXT("Small bin size must be small enough to fit into a page"));
		checkf(SmallBinSizes[Index] % UE_MBC_MIN_SMALL_POOL_ALIGNMENT == 0, TEXT("Small bin size must be a multiple of UE_MBC_MINIMUM_ALIGNMENT"));

		SmallPoolTables[Index].BinSize = SmallBinSizes[Index];
	}

	// Set up pool mappings
	uint8* IndexEntry = MemSizeToPoolIndex;
	uint32 PoolIndex  = 0;
	for (uint32 Index = 0; Index != SIZE_TO_POOL_INDEX_NUM; ++Index)
	{
		const uint32 BinSize = Index << UE_MBC_BIN_SIZE_SHIFT; // inverse of int32 Index = int32((Size >> UE_MBC_BIN_SIZE_SHIFT));
		while (SmallBinSizes[PoolIndex] < BinSize)
		{
			++PoolIndex;
			check(PoolIndex != UE_MB4_SMALL_POOL_COUNT);
		}
		check(PoolIndex < 256);
		*IndexEntry++ = uint8(PoolIndex);
	}

	AllocateHashBuckets();

	BlockPoolL1Table = (uint8**)AllocateMetaDataMemory(65536 * sizeof(uint8*));
	FMemory::Memzero(BlockPoolL1Table, 65536 * sizeof(uint8*));

	MallocBinned4 = this;
	GFixedMallocLocationPtr = (FMalloc**)(&MallocBinned4);
}

FMallocBinned4::~FMallocBinned4()
{
	for (int32 i = 0; i < 65536; ++i)
	{
		if (BlockPoolL1Table[i])
		{
			FreeMetaDataMemory(BlockPoolL1Table[i], 65536);
		}
	}
	FreeMetaDataMemory(BlockPoolL1Table, 65536 * sizeof(uint8*));
}

void FMallocBinned4::OnMallocInitialized()
{
	TMallocBinnedCommon::OnMallocInitialized();

#if UE_USE_VERYLARGEPAGEALLOCATOR
	FCoreDelegates::GetLowLevelAllocatorMemoryTrimDelegate().AddLambda([this]()
		{
			UE::TUniqueLock Lock(ExternalAllocMutex);
			CachedOSPageAllocator.FreeAll(&ExternalAllocMutex);
		}
	);

	FCoreDelegates::GetRefreshLowLevelAllocatorDelegate().AddLambda([this]()
		{
			UE::TUniqueLock Lock(ExternalAllocMutex);
			CachedOSPageAllocator.Refresh();
		}
	);
#endif
}

void FMallocBinned4::OnPreFork()
{
#if BINNED4_FORK_SUPPORT
	// Trim caches so we don't use them in the child process and cause pages to be copied
	if (GMallocBinnedPerThreadCaches)
	{
		FMallocBinnedCommonUtils::FlushCurrentThreadCache(*this);
		FMallocBinned4::Private::CheckThreadFreeBlockListsForFork();
	}

	for (int32 PoolIndex = 0; PoolIndex < UE_ARRAY_COUNT(SmallPoolTables); ++PoolIndex)
	{
		while (FBundleNode* Node = FMallocBinned4::Private::GGlobalRecycler.PopBundle(PoolIndex))
		{
			FMallocBinned4::Private::FreeBundles(*this, Node, PoolIndex);
		}
	}

#if !UE_USE_VERYLARGEPAGEALLOCATOR
	UE::TUniqueLock Lock(ExternalAllocMutex);
	CachedOSPageAllocator.FreeAll(&ExternalAllocMutex);
#endif

#endif // BINNED4_FORK_SUPPORT
}

void FMallocBinned4::OnPostFork()
{
#if BINNED4_FORK_SUPPORT
	if (GMallocBinnedPerThreadCaches)
	{
		FMallocBinnedCommonUtils::FlushCurrentThreadCache(*this);
		FMallocBinned4::Private::CheckThreadFreeBlockListsForFork();
	}

	UE_CLOGF(CurrentCanary != EBlockCanary::PreFork, LogMemory, Fatal, "FMallocBinned4 only supports forking once!");

	OldCanary = CurrentCanary;
	CurrentCanary = EBlockCanary::PostFork;

	// Mark all currently registered small-pool blocks as pre-fork (bit 7) in the side table
	{
		UE::TUniqueLock Lock(ExternalAllocMutex);
		for (int32 i = 0; i < 65536; ++i)
		{
			uint8* L2 = BlockPoolL1Table[i];
			if (L2)
			{
				for (int32 j = 0; j < 65536; ++j)
				{
					if (L2[j] != UnregisteredBlock)
					{
						L2[j] |= PreForkFlag;
					}
				}
			}
		}
	}

	for (FPoolTable& Table : SmallPoolTables)
	{
		UE::TUniqueLock Lock(Table.Mutex);
		// Clear our list of partially used pages so we don't dirty them and cause them to become unshared with the parent process
		Table.ActivePools.Clear();
		Table.ExhaustedPools.Clear();
	}
#endif
}

bool FMallocBinned4::IsInternallyThreadSafe() const
{ 
	return true;
}

FORCEINLINE static bool MB4UseSmallAlloc(SIZE_T Size, uint32 Alignment)
{
	return ((Size <= UE_MB4_MAX_SMALL_POOL_SIZE) & (Alignment <= UE_MBC_STANDARD_ALIGNMENT)); // one branch, not two
}

void* FMallocBinned4::Malloc(SIZE_T Size, uint32 Alignment)
{
	// Only allocate from the small pools if the size is small enough and the alignment isn't crazy large.
	// With large alignments, we'll waste a lot of memory allocating an entire page, but such alignments are highly unlikely in practice.
	bool bUseSmallPool = MB4UseSmallAlloc(Size, Alignment);
	if (!bUseSmallPool)
	{
		bUseSmallPool = PromoteToLargerBin(Size, Alignment, *this);
	}

	if (bUseSmallPool)
	{
		return MallocExternalSmall(Size, Alignment);
	}
	return MallocExternalLarge(Size, Alignment);
}

void* FMallocBinned4::MallocExternalSmall(SIZE_T Size, uint32 Alignment)
{
	const uint32 PoolIndex = BoundSizeToPoolIndex(Size, MemSizeToPoolIndex);
	FPerThreadFreeBlockLists* Lists = GMallocBinnedPerThreadCaches ? FPerThreadFreeBlockLists::Get() : nullptr;
	if (Lists)
	{
		if (Lists->ObtainRecycledPartial(PoolIndex, Private::GGlobalRecycler))
		{
			if (void* Result = Lists->Malloc(PoolIndex))
			{
#if UE_MB4_ALLOCATOR_STATS
				const uint32 BinSize = PoolIndexToBinSize(PoolIndex);
				Lists->AllocatedMemory += BinSize;
#endif
				return Result;
			}
		}
	}

	NOALLOC_SCOPE_CYCLE_COUNTER(STAT_FMallocBinned4_MallocExternalSmall);

	// Allocate from small object pool.
	FPoolTable& Table = SmallPoolTables[PoolIndex];

	UE::TUniqueLock Lock(Table.Mutex);

	FPoolInfo* Pool;
	if (!Table.ActivePools.IsEmpty())
	{
		Pool = &Table.ActivePools.GetFrontPool();
	}
	else
	{
		Pool = &Table.ActivePools.PushNewPoolToFront(*this, Table, PoolIndex);
	}

	void* Result = Pool->AllocateBin();
#if UE_MB4_ALLOCATOR_STATS
	++Table.TotalUsedBins;
	AllocatedSmallPoolMemory.fetch_add(Table.BinSize, std::memory_order_relaxed);
#endif

#if UE_MBC_LIGHTWEIGHT_BIN_CALLSTACK_TRACKER
	TrackBinAllocation(Table.BinSize, Table.TotalAllocatedMem);
#endif

	if (GMallocBinnedAllocExtra)
	{
		if (Lists)
		{
			// prefill the free list with some allocations so we are less likely to hit this slow path with the mutex 
			for (int32 Index = 0; Index < GMallocBinnedAllocExtra && Pool->HasFreeBin(); Index++)
			{
				if (!Lists->Free(Result, PoolIndex, Table.BinSize))
				{
					break;
				}
				Result = Pool->AllocateBin();
				UE_MBC_UPDATE_STATS(++Table.TotalUsedBins);
			}
		}
	}
	if (!Pool->HasFreeBin())
	{
		Table.ExhaustedPools.LinkToFront(Pool);
	}

	return Result;
}

void* FMallocBinned4::MallocExternalLarge(SIZE_T Size, uint32 Alignment)
{
	Alignment = FMath::Max<uint32>(Alignment, UE_MBC_STANDARD_ALIGNMENT);
	Size = Align(FMath::Max((SIZE_T)1, Size), Alignment);

	checkf(FMath::IsPowerOfTwo(Alignment), TEXT("Invalid Malloc alignment: '%u' is not a power of two"), Alignment);
	checkf(Alignment <= GMB4PageSize, TEXT("Invalid Malloc alignment: '%u' is greater than the page size '%u'"), Alignment, GMB4PageSize);
	
	const SIZE_T AlignedSize = Align(Size, OsAllocationGranularity);
	checkf(MB4IsSupportedSize(AlignedSize), TEXT("Invalid Malloc size: '%" SIZE_T_FMT "'"), AlignedSize);

	LLM_PLATFORM_SCOPE(ELLMTag::FMalloc);

	FPoolInfo* Pool;
	void*      Result;
	{
		NOALLOC_SCOPE_CYCLE_COUNTER(STAT_FMallocBinned4_MallocExternalLarge);

		UE::TUniqueLock Lock(ExternalAllocMutex);

		// Use OS for non-pooled allocations.
		LogLargeAllocation(AlignedSize);
		Result = CachedOSPageAllocator.Allocate(AlignedSize, 0, &ExternalAllocMutex);
		if (!Result)
		{
			ExternalAllocMutex.Unlock();
			OutOfMemory(AlignedSize);  // OutOfMemory is fatal and does not return
		}

		Pool = Internal::GetOrCreatePoolInfo(*this, Result, FPoolInfo::ECanary::FirstFreeBlockIsOSAllocSize);
	}

	UE_CLOGF(!IsAligned(Result, Alignment) ,LogMemory, Fatal, "FMallocBinned4 alignment was too large for OS. Alignment=%d   Ptr=%p", Alignment, Result);
	check(IsAligned(Result, GMB4PageSize) && IsOSAllocation(Result));

#if UE_MB4_ALLOCATOR_STATS
	AllocatedLargePoolMemory.fetch_add(Size, std::memory_order_relaxed);
	AllocatedLargePoolMemoryWAlignment.fetch_add(AlignedSize, std::memory_order_relaxed);
#endif

	// Create pool.
	check(Size > 0 && AlignedSize >= OsAllocationGranularity);
	Pool->SetOSAllocationSizes(Size, AlignedSize);

	return Result;
}

void* FMallocBinned4::Realloc(void* Ptr, SIZE_T NewSize, uint32 Alignment)
{
	if (NewSize == 0)
	{
		FMallocBinned4::Free(Ptr);
		return nullptr;
	}

	checkf(FMath::IsPowerOfTwo(Alignment), TEXT("Invalid Realloc alignment: '%u' is not a power of two"), Alignment);

	const uint8 ReallocEntry = GetSmallBinEntry(Ptr);
	if (ReallocEntry != UnregisteredBlock)
	{
		// Reallocate to a smaller/bigger pool if necessary
		const uint32 PoolIndex = ReallocEntry & PoolIndexMask;
		CA_ASSUME(PoolIndex < UE_MB4_SMALL_POOL_COUNT);
		const uint32 BinSize   = SmallPoolTables[PoolIndex].BinSize;
#if BINNED4_FORK_SUPPORT
		// If this is a pre-fork page, we should not allow this allocation to grow in-place to avoid copying a page from the parent process.
		if (!(ReallocEntry & PreForkFlag) &&
#else
		if(
#endif
			((NewSize <= BinSize) & (IsAligned(BinSize, Alignment))) && // one branch, not two
			(PoolIndex == 0 || NewSize > PoolIndexToBinSize(PoolIndex - 1)))
		{
			return Ptr;
		}

		// Reallocate and copy the data across
		void* Result = FMallocBinned4::Malloc(NewSize, Alignment);
		CA_ASSUME(Ptr);
		FMemory::Memcpy(Result, Ptr, FMath::Min<SIZE_T>(NewSize, BinSize));
		FMallocBinned4::Free(Ptr);
		return Result;
	}
	if (!Ptr)
	{
		void* Result = FMallocBinned4::Malloc(NewSize, Alignment);
		return Result;
	}

	NOALLOC_SCOPE_CYCLE_COUNTER(STAT_FMallocBinned4_ReallocExternal);

	// Allocated from OS.
	ExternalAllocMutex.Lock();
	FPoolInfo* Pool = Internal::FindPoolInfo(*this, Ptr);
	UE_CLOGF(!Pool, LogMemory, Fatal, "FMallocBinned4 Attempt to realloc an unrecognized pointer %p", Ptr);

	const UPTRINT PoolOsBytes = Pool->GetOsAllocatedBytes();
	const SIZE_T PoolOSRequestedBytes = Pool->GetOSRequestedBytes();
	checkf(PoolOSRequestedBytes <= PoolOsBytes, TEXT("FMallocBinned4::Realloc %d %d"), int32(PoolOSRequestedBytes), int32(PoolOsBytes));
	bool bUseSmallMalloc = MB4UseSmallAlloc(NewSize, Alignment);
	if (!bUseSmallMalloc)
	{
		bUseSmallMalloc = PromoteToLargerBin(NewSize, Alignment, *this);
	}

	if (NewSize > PoolOsBytes || // can't fit in the old block
		bUseSmallMalloc || // can switch to the small bin allocator
		Align(NewSize, OsAllocationGranularity) < PoolOsBytes) // we can get some pages back
	{
		// Grow or shrink.
		void* Result;
		if (bUseSmallMalloc)
		{
			// Unlock before a small alloc, which rarely takes a lock
			ExternalAllocMutex.Unlock();
			Result = MallocExternalSmall(NewSize, Alignment);
		}
		else
		{
			// Unlock after a large alloc, which does take a lock, to save unlocking and re-locking unnecessarily
			Result = MallocExternalLarge(NewSize, Alignment);
			ExternalAllocMutex.Unlock();
		}

		FMemory::Memcpy(Result, Ptr, FMath::Min(NewSize, PoolOSRequestedBytes));
		FMallocBinned4::Free(Ptr);
		return Result;
	}

	ExternalAllocMutex.Unlock();

	Alignment = FMath::Max<uint32>(Alignment, UE_MBC_STANDARD_ALIGNMENT);
	NewSize = Align(FMath::Max((SIZE_T)1, NewSize), Alignment);

	checkf(Alignment <= GMB4PageSize, TEXT("Invalid Realloc alignment: '%u' is greater than the page size '%u'"), Alignment, GMB4PageSize);
	checkf(MB4IsSupportedSize(NewSize), TEXT("Invalid Realloc size: '%" SIZE_T_FMT "'"), NewSize);

	UE_MBC_UPDATE_STATS(AllocatedLargePoolMemory.fetch_add((int64)NewSize - (int64)PoolOSRequestedBytes, std::memory_order_relaxed));

	Pool->SetOSAllocationSizes(NewSize, PoolOsBytes);

	return Ptr;
}

void FMallocBinned4::Free(void* Ptr)
{
	NOALLOC_SCOPE_CYCLE_COUNTER(STAT_FMallocBinned4_FreeExternal);

	const uint8 Entry = GetSmallBinEntry(Ptr);
	if (Entry != UnregisteredBlock)
	{
#if BINNED4_FORK_SUPPORT
		if (Entry & PreForkFlag)
		{
			// This page was allocated before we forked so we want to avoid dirtying it by writing a linked list into it
			return;
		}
#endif
		const uint32 PoolIndex = Entry & PoolIndexMask;
		CA_ASSUME(PoolIndex < UE_MB4_SMALL_POOL_COUNT);
		const uint32 BinSize   = SmallPoolTables[PoolIndex].BinSize;

		FBundleNode* BundlesToRecycle = nullptr;
		FPerThreadFreeBlockLists* Lists = GMallocBinnedPerThreadCaches ? FPerThreadFreeBlockLists::Get() : nullptr;
		if (Lists)
		{
			BundlesToRecycle = Lists->RecycleFullBundle(PoolIndex, Private::GGlobalRecycler);
			const bool bPushed = Lists->Free(Ptr, PoolIndex, BinSize);
			check(bPushed);
			UE_MBC_UPDATE_STATS(Lists->AllocatedMemory -= BinSize);
		}
		else
		{
			BundlesToRecycle = (FBundleNode*)Ptr;
			BundlesToRecycle->SetNextNodeInCurrentBundle(nullptr);
		}

		if (BundlesToRecycle)
		{
			Private::FreeBundles(*this, BundlesToRecycle, PoolIndex);
#if UE_MB4_ALLOCATOR_STATS
			if (!Lists)
			{
				// lists track their own stat track them instead in the global stat if we don't have lists
				AllocatedSmallPoolMemory.fetch_sub((int64)BinSize, std::memory_order_relaxed);
			}
#endif
		}
	}
	else if (Ptr)
	{
		UE::TUniqueLock Lock(ExternalAllocMutex);
		FPoolInfo* Pool = Internal::FindPoolInfo(*this, Ptr);
		if (!Pool)
		{
			UE_LOGF(LogMemory, Fatal, "FMallocBinned4 Attempt to free an unrecognized pointer %p", Ptr);
		}
		const UPTRINT PoolOsBytes = Pool->GetOsAllocatedBytes();
		const SIZE_T PoolOSRequestedBytes = Pool->GetOSRequestedBytes();

#if UE_MB4_ALLOCATOR_STATS
		AllocatedLargePoolMemory.fetch_sub((int64)PoolOSRequestedBytes, std::memory_order_relaxed);
		AllocatedLargePoolMemoryWAlignment.fetch_sub((int64)PoolOsBytes, std::memory_order_relaxed);
#endif

		checkf(PoolOSRequestedBytes <= PoolOsBytes, TEXT("FMallocBinned4::Free %d %d"), int32(PoolOSRequestedBytes), int32(PoolOsBytes));
		Pool->SetCanary(FPoolInfo::ECanary::Unassigned, true, false);
		// Free an OS allocation.
		CachedOSPageAllocator.Free(Ptr, PoolOsBytes, &ExternalAllocMutex, FPerThreadFreeBlockLists::Get() != nullptr && GBinned4MoveOSFreesOffTimeCriticalThreads != 0);
	}
}

void FMallocBinned4::FPoolList::ValidateActivePools() const
{
	for (const FPoolInfo* const* PoolPtr = &Front; *PoolPtr; PoolPtr = &(*PoolPtr)->Next)
	{
		const FPoolInfo* Pool = *PoolPtr;
		check(Pool->PtrToPrevNext == PoolPtr);
		check(Pool->FirstFreeBlock);
		for (FFreeBlock* Free = Pool->FirstFreeBlock; Free; Free = Free->GetNextFreeBlock())
		{
			check(Free->GetNumFreeBins() > 0);
		}
	}
}

void FMallocBinned4::FPoolList::ValidateExhaustedPools() const
{
	for (const FPoolInfo* const* PoolPtr = &Front; *PoolPtr; PoolPtr = &(*PoolPtr)->Next)
	{
		const FPoolInfo* Pool = *PoolPtr;
		check(Pool->PtrToPrevNext == PoolPtr);
		check(!Pool->FirstFreeBlock);
	}
}

bool FMallocBinned4::ValidateHeap()
{
	NOALLOC_SCOPE_CYCLE_COUNTER(STAT_FMallocBinned4_ValidateHeap);
	
	for (FPoolTable& Table : SmallPoolTables)
	{
		UE::TUniqueLock Lock(Table.Mutex);
		Table.ActivePools.ValidateActivePools();
		Table.ExhaustedPools.ValidateExhaustedPools();
	}

	return true;
}

const TCHAR* FMallocBinned4::GetDescriptiveName()
{
	return TEXT("Binned4");
}

void FMallocBinned4::FreeBundles(FBundleNode* Bundles, uint32 PoolIndex)
{
	Private::FreeBundles(*this, Bundles, PoolIndex);
}

void FMallocBinned4::Trim(bool bTrimThreadCaches)
{
	if (GMallocBinnedPerThreadCaches && bTrimThreadCaches)
	{
		FMallocBinnedCommonUtils::Trim(*this);

#if !UE_USE_VERYLARGEPAGEALLOCATOR
		UE::TUniqueLock Lock(ExternalAllocMutex);
		// this cache is recycled anyway, if you need to trim it based on being OOM, it's already too late.
		CachedOSPageAllocator.FreeAll(&ExternalAllocMutex);
#endif
		//UE_LOGF(LogTemp, Display, "Trim CachedOSPageAllocator = %6.2fms", 1000.0f * float(FPlatformTime::Seconds() - StartTime));
	}
}

void FMallocBinned4::FlushCurrentThreadCacheInternal(bool bNewEpochOnly)
{
	FMallocBinnedCommonUtils::FlushCurrentThreadCache(*this, bNewEpochOnly);
}

void FMallocBinned4::CanaryFail(const void* Ptr) const
{
#if BINNED4_FORK_SUPPORT
	UE_LOGF(LogMemory, Fatal, "FMallocBinned4 Attempt to free/realloc an unrecognized block %p  (no side table entry - expected pre-fork or post-fork registered block)", Ptr);
#else
	UE_LOGF(LogMemory, Fatal, "FMallocBinned4 Attempt to free/realloc an unrecognized block %p  (no side table entry)", Ptr);
#endif
}

void FMallocBinned4::RegisterBlockPool(void* BlockBase, uint32 PoolIndex)
{
	const uint64 Addr = uint64(BlockBase);
	uint8*& L2 = BlockPoolL1Table[Addr >> 32];
	if (!L2)
	{
		L2 = (uint8*)AllocateMetaDataMemory(65536);
		FMemory::Memset(L2, UnregisteredBlock, 65536);
	}
	L2[(Addr >> 16) & 0xFFFF] = uint8(PoolIndex);
}

void FMallocBinned4::UnregisterBlockPool(void* BlockBase)
{
	const uint64 Addr = uint64(BlockBase);
	uint8* L2 = BlockPoolL1Table[Addr >> 32];
	if (L2) L2[(Addr >> 16) & 0xFFFF] = UnregisteredBlock;
}

uint8 FMallocBinned4::GetSmallBinEntry(const void* Ptr) const
{
	const uint8 Entry = LookupBlockPoolEntry(Ptr);
	if (!IsAligned(Ptr, UE_MB4_LARGE_ALLOC) || Entry != UnregisteredBlock)
	{
		check(Ptr); // null is 64KB-aligned so it should never reach this branch
		if (Entry == UnregisteredBlock) CanaryFail(Ptr);
		return Entry;
	}
	return UnregisteredBlock;
}

void FMallocBinned4::DumpAllocatorStats(class FOutputDevice& Ar)
{
#if UE_MB4_ALLOCATOR_STATS

	const int64  TotalAllocatedSmallPoolMemory           = GetTotalAllocatedSmallPoolMemory();
	const int64  LocalAllocatedLargePoolMemory           = AllocatedLargePoolMemory.load(std::memory_order_relaxed);
	const int64  LocalAllocatedLargePoolMemoryWAlignment = AllocatedLargePoolMemoryWAlignment.load(std::memory_order_relaxed);
	const uint64 OSPageAllocatorCachedFreeSize           = CachedOSPageAllocator.GetCachedFreeTotal();

	Ar.Logf(TEXT("FMallocBinned4 Mem report"));
	Ar.Logf(TEXT("Constants.BinnedPageSize = %d"), int32(GMB4PageSize));
	Ar.Logf(TEXT("Constants.BinnedAllocationGranularity = %d"), int32(OsAllocationGranularity));
	Ar.Logf(TEXT("Small Pool Allocations: %fmb  (including bin size padding)"), ((double)TotalAllocatedSmallPoolMemory) / (1024.0f * 1024.0f));
	Ar.Logf(TEXT("Small Pool OS Allocated: %fmb"), ((double)AllocatedOSSmallPoolMemory.load(std::memory_order_relaxed)) / (1024.0f * 1024.0f));
	Ar.Logf(TEXT("Large Pool Requested Allocations: %fmb"), ((double)AllocatedLargePoolMemory) / (1024.0f * 1024.0f));
	Ar.Logf(TEXT("Large Pool OS Allocated: %fmb"), ((double)AllocatedLargePoolMemoryWAlignment) / (1024.0f * 1024.0f));
	Ar.Logf(TEXT("Requested Allocations: %fmb"), ((double)LocalAllocatedLargePoolMemory) / (1024.0f * 1024.0f));
	Ar.Logf(TEXT("OS Allocated: %fmb"), ((double)LocalAllocatedLargePoolMemoryWAlignment) / (1024.0f * 1024.0f));
	Ar.Logf(TEXT("PoolInfo: %fmb"), ((double)PoolInfoMemory) / (1024.0f * 1024.0f));
	Ar.Logf(TEXT("Hash: %fmb"), ((double)Binned4HashMemory) / (1024.0f * 1024.0f));
	Ar.Logf(TEXT("TLS: %fmb"), ((double)TLSMemory.load(std::memory_order_relaxed)) / (1024.0f * 1024.0f));
	Ar.Logf(TEXT("Total allocated from OS: %fmb"),
		((double)
			AllocatedOSSmallPoolMemory + AllocatedLargePoolMemoryWAlignment + PoolInfoMemory + Binned4HashMemory + TLSMemory.load(std::memory_order_relaxed)
			) / (1024.0f * 1024.0f));
	Ar.Logf(TEXT("Cached free OS pages: %fmb"), ((double)OSPageAllocatorCachedFreeSize) / (1024.0f * 1024.0f));

	for (int32 i = 0; i < UE_MB4_SMALL_POOL_COUNT; i++)
	{
		const float Fragmentation = (SmallPoolTables[i].TotalAllocatedBins > 0) ? 1.0f - (float)SmallPoolTables[i].TotalUsedBins / (float)SmallPoolTables[i].TotalAllocatedBins : 0;
		const float TotalMem = (float)SmallPoolTables[i].TotalAllocatedMem / 1024.0f / 1024.0f;
		Ar.Logf(TEXT("Bin %6d Fragmentation %d %%, Wasted Mem %.2f MB, Total Allocated Mem %.2f MB"),
				PoolIndexToBinSize(i), int(Fragmentation * 100.0f), TotalMem * Fragmentation, TotalMem);
	}

#if !PLATFORM_UNIX && !PLATFORM_ANDROID
	// Doesn't implemented
#else
	CachedOSPageAllocator.DumpAllocatorStats(Ar);
#endif

#else
	Ar.Logf(TEXT("Allocator Stats for Binned4 are not in this build. Define UE_MBC_ALLOCATOR_STATS for your project"));
#endif
}

void FMallocBinned4::UpdateStats()
{
	//Report total cached free memory in the COSPA and separately report memory that can be immediately freed back the kernel at any time
	CSV_CUSTOM_STAT(FMemory, AllocatorCachedSlackMB, (int32)(CachedOSPageAllocator.GetCachedFreeTotal() / (1024 * 1024)), ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT(FMemory, AllocatorImmediatelyFreeableCachedSlackMB, (int32)(CachedOSPageAllocator.GetCachedImmediatelyFreeable() / (1024 * 1024)), ECsvCustomStatOp::Set);

	FMallocBinnedCommonUtils::UpdateStatsCommon(*this);
	CachedOSPageAllocator.UpdateStats();
	FScopedVirtualMallocTimer::UpdateStats();
}

void* FMallocBinned4::AllocateMetaDataMemory(SIZE_T Size)
{
	LLM_PLATFORM_SCOPE(ELLMTag::FMalloc);
	Size = Align(Size, OsAllocationGranularity);
	return FPlatformMemory::BinnedAllocFromOS(Size);
}

void FMallocBinned4::FreeMetaDataMemory(void* Ptr, SIZE_T Size)
{
	if (Ptr)
	{
		Size = Align(Size, FMallocBinned4::OsAllocationGranularity);
		FPlatformMemory::BinnedFreeToOS(Ptr, Size);
	}
}


PRAGMA_RESTORE_UNSAFE_TYPECAST_WARNINGS
