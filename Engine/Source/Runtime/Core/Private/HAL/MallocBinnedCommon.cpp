// Copyright Epic Games, Inc. All Rights Reserved.

#include "HAL/MallocBinnedCommon.h"
#include "Algo/Sort.h"
#include "Containers/ArrayView.h"
#include "Misc/AssertionMacros.h"
#include "Math/NumericLimits.h"
#include "HAL/MemoryMisc.h"
#include "HAL/IConsoleManager.h"
#include "Templates/AlignmentTemplates.h"
#include "Templates/UnrealTemplate.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "Logging/LogMacros.h"

#if UE_MBC_LOG_LARGE_ALLOCATION || UE_MBC_LIGHTWEIGHT_BIN_CALLSTACK_TRACKER
	#include "HAL/PlatformStackWalk.h"
	#include "Misc/CoreDelegates.h"
#endif

PRAGMA_DISABLE_UNSAFE_TYPECAST_WARNINGS

CSV_DEFINE_CATEGORY(MallocBinned, false);

static_assert(DEFAULT_ALIGNMENT <= UE_MBC_MIN_SMALL_POOL_ALIGNMENT, "DEFAULT_ALIGNMENT is assumed to be zero");
static_assert(sizeof(FMallocBinnedCommonBase::FBundleNode) <= UE_MBC_MIN_BIN_SIZE, "FBundleNode must fit into the smallest bin size");

// Bin sizes are based around getting the maximum amount of allocations per block, with as little alignment waste as possible.
// Bin sizes should be close to even divisors of the system page size, and well distributed.
// They must be 16-byte aligned as well.
static constexpr uint32 BinnedCommonSmallBinSizes4k[] =
{
	8, 16, // +8
	32, 48, 64, 80, 96, 112, 128, 144, 160, 176, 192, // +16
	224, 256, 288, 320, // +32
	368,  // /11 ish
	400,  // /10 ish
	448,  // /9 ish
	512,  // /8
	576,  // /7 ish
	672,  // /6 ish
	816,  // /5 ish 
	1024, // /4 
	1360, // /3 ish
	2048, // /2
	4096 // /1
};

static constexpr uint32 BinnedCommonSmallBinSizes8k[] =
{
	736,  // /11 ish
	1168, // /7 ish
	1632, // /5 ish
	2720, // /3 ish
	8192  // /1
};

static constexpr uint32 BinnedCommonSmallBinSizes12k[] =
{
	//1104, // /11 ish
	//1216, // /10 ish
	1536, // /8
	1744, // /7 ish
	2448, // /5 ish
	3072, // /4
	6144, // /2
	12288 // /1
};

static constexpr uint32 BinnedCommonSmallBinSizes16k[] =
{
	//1488, // /11 ish
	//1808, // /9 ish
	//2336, // /7 ish
	3264, // /5 ish
	5456, // /3 ish
	16384 // /1
};

static constexpr uint32 BinnedCommonSmallBinSizes20k[] =
{
	// 2912, // /7 ish
	//3408, // /6 ish
	5120, // /4
	//6186, // /3 ish
	10240, // /2
	20480  // /1
};

static constexpr uint32 BinnedCommonSmallBinSizes24k[] = // 1 total
{
	24576  // /1
};

static constexpr uint32 BinnedCommonSmallBinSizes28k[] = // 6 total
{
	4768,  // /6 ish
	5728,  // /5 ish
	7168,  // /4
	9552,  // /3
	14336, // /2
	28672  // /1
};

FSizeTableEntry::FSizeTableEntry(uint32 InBinSize, uint64 PlatformPageSize, uint8 Num4kbPages, uint32 BasePageSize)
	: BinSize(InBinSize)
{
	check((PlatformPageSize & (BasePageSize - 1)) == 0 && PlatformPageSize >= BasePageSize);
	checkf(InBinSize % UE_MBC_MIN_BIN_SIZE == 0, TEXT("Small bin size must be a multiple of UE_MBC_MIN_BIN_SIZE"));

	const uint64 NumBasePagesPerPlatformPage = PlatformPageSize / BasePageSize;

	NumMemoryPagesPerBlock = 0;
	while (true)
	{
		check(NumMemoryPagesPerBlock < MAX_uint8);
		NumMemoryPagesPerBlock++;
		if (NumMemoryPagesPerBlock * NumBasePagesPerPlatformPage < Num4kbPages)
		{
			continue;
		}
		if (NumMemoryPagesPerBlock * NumBasePagesPerPlatformPage % Num4kbPages != 0)
		{
			continue;
		}
		break;
	}
	check((PlatformPageSize * NumMemoryPagesPerBlock) / BinSize <= MAX_uint32);
}

static inline void FillTable(FSizeTableEntry* SizeTable, int32& Index, const uint32* BinsList, uint32 BinListSize, uint32 Num4kbPages, uint64 PlatformPageSize, uint32 BasePageSize)
{
	for (uint32 Sub = 0; Sub < BinListSize; Sub++)
	{
		// if we override UE_MBC_MAX_LISTED_SMALL_POOL_SIZE externally, we need to filter out predefined bins of a larger size
		if (BinsList[Sub] <= UE_MBC_MAX_LISTED_SMALL_POOL_SIZE)
		{
			SizeTable[Index++] = FSizeTableEntry(BinsList[Sub], PlatformPageSize, Num4kbPages, BasePageSize);
		}
	}
}

uint8 FSizeTableEntry::FillSizeTable(uint64 PlatformPageSize, FSizeTableEntry* SizeTable, uint32 BasePageSize, uint32 MaxSize, uint32 SizeIncrement)
{
	check(BasePageSize <= UE_MBC_MAX_SUPPORTED_PLATFORM_PAGE_SIZE);

	int32 Index = 0;
	FillTable(SizeTable, Index, BinnedCommonSmallBinSizes4k,  UE_ARRAY_COUNT(BinnedCommonSmallBinSizes4k),  1, PlatformPageSize, BasePageSize);
	FillTable(SizeTable, Index, BinnedCommonSmallBinSizes8k,  UE_ARRAY_COUNT(BinnedCommonSmallBinSizes8k),  2, PlatformPageSize, BasePageSize);
	FillTable(SizeTable, Index, BinnedCommonSmallBinSizes12k, UE_ARRAY_COUNT(BinnedCommonSmallBinSizes12k), 3, PlatformPageSize, BasePageSize);
	FillTable(SizeTable, Index, BinnedCommonSmallBinSizes16k, UE_ARRAY_COUNT(BinnedCommonSmallBinSizes16k), 4, PlatformPageSize, BasePageSize);
	FillTable(SizeTable, Index, BinnedCommonSmallBinSizes20k, UE_ARRAY_COUNT(BinnedCommonSmallBinSizes20k), 5, PlatformPageSize, BasePageSize);
	FillTable(SizeTable, Index, BinnedCommonSmallBinSizes24k, UE_ARRAY_COUNT(BinnedCommonSmallBinSizes24k), 6, PlatformPageSize, BasePageSize);
	FillTable(SizeTable, Index, BinnedCommonSmallBinSizes28k, UE_ARRAY_COUNT(BinnedCommonSmallBinSizes28k), 7, PlatformPageSize, BasePageSize);

	check(Index == UE_MBC_NUM_LISTED_SMALL_POOLS);

	Algo::Sort(MakeArrayView(SizeTable, Index));
	check(SizeTable[Index - 1].BinSize == UE_MBC_MAX_LISTED_SMALL_POOL_SIZE);
	check((UE_MBC_MAX_LISTED_SMALL_POOL_SIZE == MaxSize) || IsAligned(UE_MBC_MAX_LISTED_SMALL_POOL_SIZE, BasePageSize));
	for (uint32 Size = UE_MBC_MAX_LISTED_SMALL_POOL_SIZE + BasePageSize; Size <= MaxSize; Size += SizeIncrement)
	{
		SizeTable[Index++] = FSizeTableEntry(Size, PlatformPageSize, Size / BasePageSize, BasePageSize);
	}
	check(Index < 256);
	return (uint8)Index;
}

void FBitTree::FBitTreeInit(uint32 InDesiredCapacity, void * Memory, uint32 MemorySize, bool InitialValue)
{
	Bits = (uint64*)Memory;
	DesiredCapacity = InDesiredCapacity;
	AllocationSize = 8;
	Rows = 1;
	uint32 RowsUint64s = 1;
	Capacity = 64;
	OffsetOfLastRow = 0;

	uint32 RowOffsets[10]; // 10 is way more than enough
	RowOffsets[0] = 0;
	uint32 RowNum[10]; // 10 is way more than enough
	RowNum[0] = 1;

	while (Capacity < DesiredCapacity)
	{
		Capacity *= 64;
		RowsUint64s *= 64;
		OffsetOfLastRow = AllocationSize / 8;
		check(Rows < 10);
		RowOffsets[Rows] = OffsetOfLastRow;
		RowNum[Rows] = RowsUint64s;
		AllocationSize += 8 * RowsUint64s;
		Rows++;
	}

	uint32 LastRowTotal = (AllocationSize - OffsetOfLastRow * 8) * 8;
	uint32 ExtraBits = LastRowTotal - DesiredCapacity;
	AllocationSize -= (ExtraBits / 64) * 8;
	check(AllocationSize <= MemorySize && Bits);

	FMemory::Memset(Bits, InitialValue ? 0xff : 0, AllocationSize);

	if (!InitialValue)
	{
		// we fill everything beyond the desired size with occupied
		uint32 ItemsPerBit = 64;
		for (int32 FillRow = Rows - 2; FillRow >= 0; FillRow--)
		{
			uint32 NeededOneBits = RowNum[FillRow] * 64 - (DesiredCapacity + ItemsPerBit - 1) / ItemsPerBit;
			uint32 NeededOne64s = NeededOneBits / 64;
			NeededOneBits %= 64;
			for (uint32 Fill = RowNum[FillRow] - NeededOne64s; Fill < RowNum[FillRow]; Fill++)
			{
				Bits[RowOffsets[FillRow] + Fill] = MAX_uint64;
			}
			if (NeededOneBits)
			{
				Bits[RowOffsets[FillRow] + RowNum[FillRow] - NeededOne64s - 1] = (MAX_uint64 << (64 - NeededOneBits));
			}
			ItemsPerBit *= 64;
		}

		if (DesiredCapacity % 64)
		{
			Bits[AllocationSize / 8 - 1] = (MAX_uint64 << (DesiredCapacity % 64));
		}
	}
}

uint32 FBitTree::AllocBit()
{
	uint32 Result = MAX_uint32;
	if (*Bits != MAX_uint64) // else we are full
	{
		Result = 0;
		uint32 Offset = 0;
		uint32 Row = 0;
		while (true)
		{
			uint64* At = Bits + Offset;
			check(At >= Bits && At < Bits + AllocationSize / 8);
			const uint32 LowestZeroBit = FMath::CountTrailingZeros64(~*At);
			check(LowestZeroBit < 64);
			Result = Result * 64 + LowestZeroBit;
			if (Row == Rows - 1)
			{
				check(!((*At) & (1ull << LowestZeroBit))); // this was already allocated?
				*At |= (1ull << LowestZeroBit);
				if (Row > 0)
				{
					if (*At == MAX_uint64)
					{
						do
						{
							const uint32 Rem = (Offset - 1) % 64;
							Offset = (Offset - 1) / 64;
							At = Bits + Offset;
							check(At >= Bits && At < Bits + AllocationSize / 8);
							check(*At != MAX_uint64); // this should not already be marked full
							*At |= (1ull << Rem);
							if (*At != MAX_uint64)
							{
								break;
							}
							Row--;
						} while (Row);
					}
				}
				break;
			}
			Offset = Offset * 64 + 1 + LowestZeroBit;
			Row++;
		}
	}

	return Result;
}

bool FBitTree::IsAllocated(uint32 Index) const
{
	check(Index < DesiredCapacity);
	const uint32 Row = Rows - 1;
	const uint32 Rem = Index % 64;
	const uint32 Offset = OffsetOfLastRow + Index / 64;
	const uint64* At = Bits + Offset;
	check(At >= Bits && At < Bits + AllocationSize / 8);
	return !!((*At) & (1ull << Rem));
}

void FBitTree::AllocBit(uint32 Index)
{
	check(Index < DesiredCapacity);
	uint32 Row = Rows - 1;
	uint32 Rem = Index % 64;
	uint32 Offset = OffsetOfLastRow + Index / 64;
	uint64* At = Bits + Offset;
	check(At >= Bits && At < Bits + AllocationSize / 8);
	check(!((*At) & (1ull << Rem))); // this was already allocated?
	*At |= (1ull << Rem);
	if (*At == MAX_uint64 && Row > 0)
	{
		do
		{
			Rem = (Offset - 1) % 64;
			Offset = (Offset - 1) / 64;
			At = Bits + Offset;
			check(At >= Bits && At < Bits + AllocationSize / 8);
			check(!((*At) & (1ull << Rem))); // this was already allocated?
			*At |= (1ull << Rem);
			if (*At != MAX_uint64)
			{
				break;
			}
			Row--;
		} while (Row);
	}

}

uint32 FBitTree::NextAllocBit() const
{
	uint32 Result = MAX_uint32;
	if (*Bits != MAX_uint64) // else we are full
	{
		Result = 0;
		uint32 Offset = 0;
		uint32 Row = 0;
		while (true)
		{
			const uint64* At = Bits + Offset;
			check(At >= Bits && At < Bits + AllocationSize / 8);
			const uint32 LowestZeroBit = FMath::CountTrailingZeros64(~*At);
			check(LowestZeroBit < 64);
			Result = Result * 64 + LowestZeroBit;
			if (Row == Rows - 1)
			{
				check(!((*At) & (1ull << LowestZeroBit))); // this was already allocated?
				break;
			}
			Offset = Offset * 64 + 1 + LowestZeroBit;
			Row++;
		}
	}

	return Result;
}

uint32 FBitTree::NextAllocBit(uint32 StartIndex) const
{
	if (*Bits != MAX_uint64) // else we are full
	{
		const uint32 Index = StartIndex;
		check(Index < DesiredCapacity);
		uint32 Row = Rows - 1;
		uint32 Rem = Index % 64;
		uint32 Offset = OffsetOfLastRow + Index / 64;
		const uint64* At = Bits + Offset;
		check(At >= Bits && At < Bits + AllocationSize / 8);
		uint64 LocalAt = *At;
		if (!(LocalAt & (1ull << Rem)))
		{
			return Index; // lucked out, start was unallocated
		}
		// start was allocated, search for an unallocated one
		LocalAt |= MAX_uint64 >> (63 - Rem); // set and ignore the bits representing items before (and including) the start
		if (LocalAt != MAX_uint64)
		{
			// this qword has an item we can use
			const uint32 LowestZeroBit = FMath::CountTrailingZeros64(~LocalAt);
			check(LowestZeroBit < 64);
			return Index - Rem + LowestZeroBit;

		}
		// rest of qword was also allocated, search up the tree for the next free item
		if (Row > 0)
		{
			do
			{
				Row--;
				Rem = (Offset - 1) % 64;
				Offset = (Offset - 1) / 64;
				At = Bits + Offset;
				check(At >= Bits && At < Bits + AllocationSize / 8);
				LocalAt = *At;
				LocalAt |= MAX_uint64 >> (63 - Rem); // set and ignore the bits representing items before (and including) the start
				if (LocalAt != MAX_uint64)
				{
					// this qword has an item we can use
					// now search down the tree
					while (true)
					{
						const uint32 LowestZeroBit = FMath::CountTrailingZeros64(~LocalAt);
						check(LowestZeroBit < 64);
						if (Row == Rows - 1)
						{
							check(!(LocalAt & (1ull << LowestZeroBit))); // this was already allocated?
							const uint32 Result = (Offset - OffsetOfLastRow) * 64 + LowestZeroBit;
							check(Result < DesiredCapacity);
							return Result;

						}
						Offset = Offset * 64 + 1 + LowestZeroBit;
						At = Bits + Offset;
						check(At >= Bits && At < Bits + AllocationSize / 8);
						LocalAt = *At;
						Row++;
					}
				}
			} while (Row);
		}
	}

	return MAX_uint32;
}

void FBitTree::FreeBit(uint32 Index)
{
	check(Index < DesiredCapacity);
	uint32 Row = Rows - 1;
	uint32 Rem = Index % 64;
	uint32 Offset = OffsetOfLastRow + Index / 64;
	uint64* At = Bits + Offset;
	check(At >= Bits && At < Bits + AllocationSize / 8);
	bool bWasFull = *At == MAX_uint64;
	check((*At) & (1ull << Rem)); // this was not already allocated?
	*At &= ~(1ull << Rem);
	if (bWasFull && Row > 0)
	{
		do
		{
			Rem = (Offset - 1) % 64;
			Offset = (Offset - 1) / 64;
			At = Bits + Offset;
			check(At >= Bits && At < Bits + AllocationSize / 8);
			bWasFull = *At == MAX_uint64;
			*At &= ~(1ull << Rem);
			if (!bWasFull)
			{
				break;
			}
			Row--;
		} while (Row);
	}
}

uint32 FBitTree::CountOnes(uint32 UpTo) const
{
	uint32 Result = 0;
	const uint64* At = Bits + OffsetOfLastRow;
	while (UpTo >= 64)
	{
		Result += FMath::CountBits64(*At);
		At++;
		UpTo -= 64;
	}
	if (UpTo)
	{
		Result += FMath::CountBits64((*At) << (64 - UpTo));
	}
	return Result;
}

/**
 * Finds a contiguous span of unallocated bits.
 * NumBits must be a power of two or a multiple of 64.
 * Only checks regions aligned to min(NumBits, 64).
 * 
 * Warning, slow!
 * Requires a linear search along the bottom row! O(Capacity / min(NumBits,64)) iterations.
 * 
 * Returns the index of the first unallocated bit in the span.
 */
uint32 FBitTree::Slow_NextAllocBits(uint32 NumBits, uint64 StartIndex)
{
	check(FMath::IsPowerOfTwo(NumBits) || NumBits % 64 == 0);

	uint32 Offset = OffsetOfLastRow + StartIndex / 64;
	uint32 MaxOffset = OffsetOfLastRow + DesiredCapacity / 64;

	// If the number of bits is >= 64, we can search int-by-int instead of bit-by-bit on the lowest row
	if (NumBits >= 64)
	{
		uint32 NumInts = NumBits / 64; // Number of uint64s required to hold our allocation
		uint32 FreeInts = 0; // Number of free uint64s we've encountered in a row
		
		while (Offset < MaxOffset)
		{
			// Increment FreeInts if the uint64 is empty, otherwise reset to 0
			FreeInts = Bits[Offset] ? 0 : (FreeInts + 1);

			if (FreeInts == NumInts)
			{
				// Offset will be pointing at the last uint64 in the free span - correct to point at the first
				return ((Offset - OffsetOfLastRow) - (NumInts - 1)) * 64;
			}
			
			Offset++;
		}
	}

	// If the number of bits is < 64, we need to loop over ints and then power of two bits within those ints
	else
	{
		uint32 SlotsPerInt = 64 / NumBits; // How many possible positions for our allocation there are per uint64

		while (Offset < MaxOffset)
		{
			// Allocation of bits within a given uint64 goes RIGHT-TO-LEFT
			// Create right-aligned mask based on number of bits
			uint64 Mask = (1ull << NumBits) - 1;

			// Shift that mask left to check each aligned section of the int
			// Return offset + bit position if we find a free section
			for (uint32 i = 0; i < SlotsPerInt; i++)
			{
				if (!(Mask & Bits[Offset]))
				{
					uint64 Result = (Offset - OffsetOfLastRow) * 64 + (i * NumBits);

					// Ensure we don't return results before our requested start index
					if (Result >= StartIndex) {
						return Result;
					}
				}

				// Shift mask left
				Mask <<= NumBits;
			}

			Offset++;
		}
	}

	check(false); // If we get here, no aligned span of this size is available
	return MAX_uint32;
}

uint32 FMallocBinnedCommonBase::OsAllocationGranularity = 0;

float GMallocBinnedFlushThreadCacheMaxWaitTime = 0.2f;
static FAutoConsoleVariableRef GMallocBinnedFlushThreadCacheMaxWaitTimeCVar(
	TEXT("MallocBinned.FlushThreadCacheMaxWaitTime"),
	GMallocBinnedFlushThreadCacheMaxWaitTime,
	TEXT("The threshold of time before warning about FlushCurrentThreadCache taking too long (seconds)."),
	ECVF_ReadOnly
);

int32 GMallocBinnedFlushRegisteredThreadCachesOnOneThread = 1;
static FAutoConsoleVariableRef GMallocBinnedFlushRegisteredThreadCachesOnOneThreadCVar(
	TEXT("MallocBinned.FlushRegisteredThreadCachesOnOneThread"),
	GMallocBinnedFlushRegisteredThreadCachesOnOneThread,
	TEXT("Whether or not to attempt to flush registered thread caches on one thread (enabled by default)."));

#if UE_MBC_SUPPORT_RUNTIME_TLC_TOGGLE
	int32 GMallocBinnedPerThreadCaches = UE_DEFAULT_GMallocBinnedPerThreadCaches;
	static FAutoConsoleVariableRef GMallocBinnedPerThreadCachesCVar(
		TEXT("MallocBinned.PerThreadCaches"),
		GMallocBinnedPerThreadCaches,
		TEXT("Enables per-thread caches of small (<= 32768 byte) allocations from FMallocBinned2/3")
	);
#endif

#if UE_MBC_ALLOW_RUNTIME_TWEAKING

	int32 GMallocBinnedBundleSize = UE_DEFAULT_GMallocBinnedBundleSize;
	static FAutoConsoleVariableRef GMallocBinnedBundleSizeCVar(
		TEXT("MallocBinned.BundleSize"),
		GMallocBinnedBundleSize,
		TEXT("Max size in bytes of per-block bundles used in the recycling process")
	);

	int32 GMallocBinnedBundleCount = UE_DEFAULT_GMallocBinnedBundleCount;
	static FAutoConsoleVariableRef GMallocBinnedBundleCountCVar(
		TEXT("MallocBinned.BundleCount"),
		GMallocBinnedBundleCount,
		TEXT("Max count in blocks per-block bundles used in the recycling process")
	);

	int32 GMallocBinnedAllocExtra = UE_DEFAULT_GMallocBinnedAllocExtra;
	static FAutoConsoleVariableRef GMallocBinnedAllocExtraCVar(
		TEXT("MallocBinned.AllocExtra"),
		GMallocBinnedAllocExtra,
		TEXT("When we do acquire the lock, how many bins cached in TLS caches. In no case will we grab more than a page.")
	);

	int32 GMallocBinnedMaxBundlesBeforeRecycle = UE_DEFAULT_GMallocBinnedMaxBundlesBeforeRecycle;
	static FAutoConsoleVariableRef GMallocBinnedMaxBundlesBeforeRecycleCVar(
		TEXT("MallocBinned.BundleRecycleCount"),
		GMallocBinnedMaxBundlesBeforeRecycle,
		TEXT("Number of freed bundles in the global recycler before it returns them to the system, per-block size. Limited by UE_DEFAULT_GBinned3MaxBundlesBeforeRecycle (currently 4)")
	);
#endif	//~UE_MBC_ALLOW_RUNTIME_TWEAKING

#if UE_MBC_ALLOCATOR_STATS
	std::atomic<int64> FMallocBinnedCommonBase::TLSMemory(0);
	std::atomic<int64> FMallocBinnedCommonBase::ConsolidatedMemory(0);

	std::atomic<int64> FMallocBinnedCommonBase::AllocatedSmallPoolMemory(0);
	std::atomic<int64> FMallocBinnedCommonBase::AllocatedOSSmallPoolMemory(0);

	std::atomic<int64> FMallocBinnedCommonBase::AllocatedLargePoolMemory(0);
	std::atomic<int64> FMallocBinnedCommonBase::AllocatedLargePoolMemoryWAlignment(0);

	int64 FMallocBinnedCommonBase::PoolInfoMemory = 0;
	int64 FMallocBinnedCommonBase::HashMemory = 0;

	int32 GMallocBinnedEnableCSVStats = 0;
	static FAutoConsoleVariableRef GMallocBinnedEnableCSVStatsCVar(
		TEXT("MallocBinned.EnableCSVStats"),
		GMallocBinnedEnableCSVStats,
		TEXT("Whether or not to enable extended CSV stats with fragmentation stats (disabled by default)."));

void FMallocBinnedCommonBase::GetAllocatorStatsInternal(FGenericMemoryStats& OutStats, int64 TotalAllocatedSmallPoolMemory)
{
	const int64 LocalAllocatedOSSmallPoolMemory = AllocatedOSSmallPoolMemory.load(std::memory_order_relaxed);
	const int64 LocalAllocatedLargePoolMemory = AllocatedLargePoolMemory.load(std::memory_order_relaxed);
	const int64 LocalAllocatedLargePoolMemoryWAlignment = AllocatedLargePoolMemoryWAlignment.load(std::memory_order_relaxed);

	OutStats.Add(TEXT("AllocatedSmallPoolMemory"), TotalAllocatedSmallPoolMemory);
	OutStats.Add(TEXT("AllocatedOSSmallPoolMemory"), LocalAllocatedOSSmallPoolMemory);
	OutStats.Add(TEXT("AllocatedLargePoolMemory"), LocalAllocatedLargePoolMemory);
	OutStats.Add(TEXT("AllocatedLargePoolMemoryWAlignment"), LocalAllocatedLargePoolMemoryWAlignment);

	const uint64 TotalAllocated = TotalAllocatedSmallPoolMemory + LocalAllocatedLargePoolMemory;
	const uint64 TotalOSAllocated = LocalAllocatedOSSmallPoolMemory + LocalAllocatedLargePoolMemoryWAlignment + GetTotalFreeCachedMemorySize();

	OutStats.Add(TEXT("TotalAllocated"), TotalAllocated);
	OutStats.Add(TEXT("TotalOSAllocated"), TotalOSAllocated);

	FMalloc::GetAllocatorStats(OutStats);
}
#endif	//~UE_MBC_ALLOCATOR_STATS

void FMallocBinnedCommonBase::UnrecognizedPointerFatalError(void* Ptr)
{
	UE_LOGF(LogMemory, Fatal, "FMallocBinnedCommon attempted to GetAllocationSizeExternal on an unrecognized pointer %p", Ptr);
	FPlatformProcess::SleepInfinite(); // Unreachable
}

#if UE_MBC_LOG_LARGE_ALLOCATION

static int32 GLogLargeAllocationSizeInMB = 0;
static FAutoConsoleVariableRef CVarLogLargeAllocationSizeInMB(
	TEXT("MallocBinned.LogLargeAllocationInMB"),
	GLogLargeAllocationSizeInMB,
	TEXT("Log large memory allocations and their callstacks if the allocation size exceeds the specified MB threshold (default is 0, meaning no allocations are logged)."));

void FMallocBinnedCommonBase::LogLargeAllocation(SIZE_T Size) const
{
	if (GLogLargeAllocationSizeInMB <= 0)
	{
		return;
	}

	const int32 AllocationSizeInMB = int32((Size / 1024) / 1024);

	if (AllocationSizeInMB < GLogLargeAllocationSizeInMB)
	{
		return;
	}

	static bool bInFunction = false;
	if (bInFunction)
	{
		return;
	}

	TGuardValue<bool> GuardInFunction(bInFunction, true);

	// Grab callstack
	static constexpr int32 MaxCallStackDepth = 16;
	static constexpr int32 CallStackEntriesToSkipCount = 2;

	uint64 FullCallStack[MaxCallStackDepth + CallStackEntriesToSkipCount] = { 0 };
	FPlatformStackWalk::CaptureStackBackTrace(&FullCallStack[0], MaxCallStackDepth + CallStackEntriesToSkipCount);

	// Skip first n entries as they are inside the allocator.
	uint64* CallStack = &FullCallStack[CallStackEntriesToSkipCount];

	UE_LOGF(LogMemory, Log, "Allocating %d MB - 0x%08llx 0x%08llx 0x%08llx 0x%08llx 0x%08llx 0x%08llx 0x%08llx 0x%08llx 0x%08llx 0x%08llx 0x%08llx 0x%08llx 0x%08llx 0x%08llx 0x%08llx 0x%08llx",
		   AllocationSizeInMB,
		   CallStack[0],
		   CallStack[1],
		   CallStack[2],
		   CallStack[3],
		   CallStack[4],
		   CallStack[5],
		   CallStack[6],
		   CallStack[7],
		   CallStack[8],
		   CallStack[9],
		   CallStack[10],
		   CallStack[11],
		   CallStack[12],
		   CallStack[13],
		   CallStack[14],
		   CallStack[15]);
}
#endif

#if UE_MBC_LIGHTWEIGHT_BIN_CALLSTACK_TRACKER
static int32 GMBCBinSizeToTrack = 0;
static FAutoConsoleVariableRef CVarGMBCBinSizeToTrack(
	TEXT("MallocBinned.LightWeightBinTracker.BinSize"),
	GMBCBinSizeToTrack,
	TEXT("BinSize to track (default is 0, meaning no allocations are tracked)."));

static int32 GMBCTotalBinSizeInMBThresholdToTrack = 0;
static FAutoConsoleVariableRef CVarGMBCTotalBinSizeInMBThresholdToTrack(
	TEXT("MallocBinned.LightWeightBinTracker.TotalBinSizeInMBThreshold"),
	GMBCTotalBinSizeInMBThresholdToTrack,
	TEXT("Track allocation callstack for bin (specified with MallocBinned.LightWeightTracker.BinSize) when its usage size is over a certain threshold in MB (default is 0, meaning no allocations are tracked)."));

struct FBinCallStackTracker
{
	void Track(uint32 BinSize, uint32 TotalAllocatedMem);
	void ForceCrash();

	static constexpr const uint32 NumFramePointers = 16;
	static constexpr const uint32 NumCallStacks = 512;

	uint64 AllocationCallStacks[NumCallStacks][NumFramePointers];
	int32 AllocationCallStacksFrame[NumCallStacks] = { -1 };

	uint32 CallStackIndex = 0;
	std::atomic<int32> CallStackFrame = 0;

	FCriticalSection Mutex;
} GMBCBinCallStackTracker;

void FMallocBinnedCommonBase::TrackBinAllocation(uint32 BinSize, uint32 TotalAllocatedMem)
{
	GMBCBinCallStackTracker.Track(BinSize, TotalAllocatedMem);
}

void FBinCallStackTracker::Track(uint32 BinSize, uint32 TotalAllocatedMem)
{
	if (GMBCBinSizeToTrack <= 0 || GMBCTotalBinSizeInMBThresholdToTrack <= 0)
	{
		return;
	}
	
	int32 TotalAllocatedMemInMb = (TotalAllocatedMem / 1024) / 1024;
	if (GMBCBinSizeToTrack != BinSize || TotalAllocatedMemInMb < GMBCTotalBinSizeInMBThresholdToTrack)
	{
		return;
	}

	static bool bInFunction = false;
	if (bInFunction)
	{
		return;
	}

	TGuardValue<bool> GuardInFunction(bInFunction, true);
	FScopeLock Lock(&Mutex);

	// Grab callstack
	static constexpr int32 CallStackEntriesToSkipCount = 2;
	static constexpr int32 MaxCallStackDepth = NumFramePointers + CallStackEntriesToSkipCount;

	uint64 FullCallStack[MaxCallStackDepth] = { 0 };
	FPlatformStackWalk::CaptureStackBackTrace(&FullCallStack[0], MaxCallStackDepth);

	uint64* CallStackSrc = &FullCallStack[CallStackEntriesToSkipCount];
	uint64* CallStackDst = &AllocationCallStacks[CallStackIndex][0];
	FMemory::Memcpy(CallStackDst, CallStackSrc, NumFramePointers * sizeof(uint64));

	AllocationCallStacksFrame[CallStackIndex] = CallStackFrame.load(std::memory_order_relaxed);

	CallStackIndex++;
	CallStackIndex = CallStackIndex % NumCallStacks;
}

// Intentionally disabling optimization and pgo profile to ensure breadcrumbing data are copied in the stack
UE_DISABLE_OPTIMIZATION_SHIP
UE_NO_PROFILE_ATTRIBUTE
void FBinCallStackTracker::ForceCrash()
{
	// No need to force crash if tracker is disabled.
	if (GMBCBinSizeToTrack <= 0 || GMBCTotalBinSizeInMBThresholdToTrack <= 0)
	{
		return;
	}
	
	FScopeLock Lock(&Mutex);

	// Allocate on stack and copy latest 32 callstack data so we could get them from a minidump
	static constexpr int NumLocalCallStacks = 32;
	static_assert(NumLocalCallStacks <= NumCallStacks);

	uint64 StackAllocationCallStacks[NumLocalCallStacks][NumFramePointers];
	int32 StackAllocationCallStacksFrame[NumLocalCallStacks];

	for (int32 Index = 0; Index < NumLocalCallStacks; Index++)
	{
		CallStackIndex--;
		CallStackIndex = CallStackIndex % NumCallStacks;

		uint64* CallStackSrc = &AllocationCallStacks[CallStackIndex][0];
		for (int32 FramePointerIndex = 0; FramePointerIndex < NumFramePointers; FramePointerIndex++)
		{
			StackAllocationCallStacks[Index][FramePointerIndex] = CallStackSrc[FramePointerIndex];
		}
		
		StackAllocationCallStacksFrame[Index] = AllocationCallStacksFrame[CallStackIndex];
	}

	UE_LOGF(LogMemory, Fatal, "OOM Force crash requested by MallocBinned Light Weight CallStack tracker, Frame %d", CallStackFrame.load(std::memory_order_relaxed));
}
// Enable back optimization
UE_ENABLE_OPTIMIZATION_SHIP
#endif

void FMallocBinnedCommonBase::OnMallocInitialized()
{
#if UE_MBC_LIGHTWEIGHT_BIN_CALLSTACK_TRACKER
	// When OnMallocInitialized is called, global static OnBeginFrame is not yet allocated on some platforms. 
	// As a workaround OnBeginFrame delegate is added via TSConfigReadyForUse which is called after global static initialization
	FCoreDelegates::TSConfigReadyForUse().AddLambda([this]()
		{
			UE_CALL_ONCE([&]()
			{
				FCoreDelegates::OnBeginFrame.AddLambda([this]()
					{
						GMBCBinCallStackTracker.CallStackFrame++;
						GMBCBinCallStackTracker.CallStackFrame &= 0xFFFF; // Wrap it around 65536 to get it more human readable
					});
			});
		});


	FCoreDelegates::GetOutOfMemoryDelegate().AddLambda([this]()
		{
			GMBCBinCallStackTracker.ForceCrash();
		});
#endif
}

PRAGMA_RESTORE_UNSAFE_TYPECAST_WARNINGS
