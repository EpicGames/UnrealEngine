// Copyright Epic Games, Inc. All Rights Reserved.

#include "HAL/MallocBinnedGPU.h"

PRAGMA_DISABLE_UNSAFE_TYPECAST_WARNINGS

#if PLATFORM_HAS_FPlatformVirtualMemoryBlock
#include "Logging/LogMacros.h"
#include "Templates/Function.h"
#include "GenericPlatform/GenericPlatformProcess.h"
#include "Stats/Stats.h"
#include "HAL/IConsoleManager.h"
#include "HAL/MemoryMisc.h"
#include "HAL/PlatformMisc.h"

#include <limits>

struct FMallocBinnedGPU::FPoolInfoSmall
{
	enum ECanary
	{
		SmallUnassigned = 0x5effeed,
		SmallAssigned = 0x69ffeed
	};

	uint32 Canary;
	uint32 Taken;
	FFreeBlock* FirstFreeProxy;

	FPoolInfoSmall()
		: Canary(ECanary::SmallUnassigned)
		, Taken(0)
		, FirstFreeProxy(nullptr)
	{
		static_assert(sizeof(FPoolInfoSmall) == 16, "Padding fail");
	}
	void CheckCanary(ECanary ShouldBe) const
	{
		if (Canary != ShouldBe)
		{
			UE_LOGF(LogMemory, Fatal, "MallocBinnedGPU Corruption Canary was 0x%x, should be 0x%x", int32(Canary), int32(ShouldBe));
		}
	}
	void SetCanary(ECanary ShouldBe, bool bPreexisting, bool bGuarnteedToBeNew)
	{
		if (bPreexisting)
		{
			if (bGuarnteedToBeNew)
			{
				UE_LOGF(LogMemory, Fatal, "MallocBinnedGPU Corruption Canary was 0x%x, should be 0x%x. This block is both preexisting and guaranteed to be new; which makes no sense.", int32(Canary), int32(ShouldBe));
			}
			if (ShouldBe == ECanary::SmallUnassigned)
			{
				if (Canary != ECanary::SmallAssigned)
				{
					UE_LOGF(LogMemory, Fatal, "MallocBinnedGPU Corruption Canary was 0x%x, will be 0x%x because this block should be preexisting and in use.", int32(Canary), int32(ShouldBe));
				}
			}
			else if (Canary != ShouldBe)
			{
				UE_LOGF(LogMemory, Fatal, "MallocBinnedGPU Corruption Canary was 0x%x, should be 0x%x because this block should be preexisting.", int32(Canary), int32(ShouldBe));
			}
		}
		else
		{
			if (bGuarnteedToBeNew)
			{
				if (Canary != ECanary::SmallUnassigned)
				{
					UE_LOGF(LogMemory, Fatal, "MallocBinnedGPU Corruption Canary was 0x%x, will be 0x%x. This block is guaranteed to be new yet is it already assigned.", int32(Canary), int32(ShouldBe));
				}
			}
			else if (Canary != ShouldBe && Canary != ECanary::SmallUnassigned)
			{
				UE_LOGF(LogMemory, Fatal, "MallocBinnedGPU Corruption Canary was 0x%x, will be 0x%x does not have an expected value.", int32(Canary), int32(ShouldBe));
			}
		}
		Canary = ShouldBe;
	}
	bool HasFreeRegularBlock() const
	{
		CheckCanary(ECanary::SmallAssigned);
		return !!FirstFreeProxy;
	}

	void* AllocateRegularBlock(FMallocBinnedGPU& Allocator, uint8 MinimumAlignmentShift)
	{
		check(HasFreeRegularBlock());
		++Taken;
		FFreeBlock* Free = FirstFreeProxy;
		void* Result = Free->AllocateRegularBlock(MinimumAlignmentShift);
		if (Free->GetNumFreeRegularBlocks() == 0)
		{
			FirstFreeProxy = Free->NextFreeBlock;
			delete (FGPUMemoryBlockProxy*)Free;
			MBG_STAT(Allocator.GPUProxyMemory -= sizeof(FGPUMemoryBlockProxy);)
		}

		return Result;
	}
};

struct FMallocBinnedGPU::FPoolInfoLarge
{
	enum ECanary
	{
		LargeUnassigned = 673,
		LargeAssigned = 3917,
	};

public:
	uint32 Canary;      
private:
	uint32 VMSizeDivVirtualSizeAlignment;
	uint32 OSCommitSize;
	uint32 AllocSize;      // Number of bytes allocated
public:

	FPoolInfoLarge() :
		Canary(ECanary::LargeUnassigned),
		VMSizeDivVirtualSizeAlignment(0),
		OSCommitSize(0),
		AllocSize(0)
	{
	}
	void CheckCanary(ECanary ShouldBe) const
	{
		if (Canary != ShouldBe)
		{
			UE_LOGF(LogMemory, Fatal, "MallocBinnedGPU Corruption Canary was 0x%x, should be 0x%x", int32(Canary), int32(ShouldBe));
		}
	}
	void SetCanary(ECanary ShouldBe, bool bPreexisting, bool bGuarnteedToBeNew)
	{
		if (bPreexisting)
		{
			if (bGuarnteedToBeNew)
			{
				UE_LOGF(LogMemory, Fatal, "MallocBinnedGPU Corruption Canary was 0x%x, should be 0x%x. This block is both preexisting and guaranteed to be new; which makes no sense.", int32(Canary), int32(ShouldBe));
			}
			if (ShouldBe == ECanary::LargeUnassigned)
			{
				if (Canary != ECanary::LargeAssigned)
				{
					UE_LOGF(LogMemory, Fatal, "MallocBinnedGPU Corruption Canary was 0x%x, will be 0x%x because this block should be preexisting and in use.", int32(Canary), int32(ShouldBe));
				}
			}
			else if (Canary != ShouldBe)
			{
				UE_LOGF(LogMemory, Fatal, "MallocBinnedGPU Corruption Canary was 0x%x, should be 0x%x because this block should be preexisting.", int32(Canary), int32(ShouldBe));
			}
		}
		else
		{
			if (bGuarnteedToBeNew)
			{
				if (Canary != ECanary::LargeUnassigned)
				{
					UE_LOGF(LogMemory, Fatal, "MallocBinnedGPU Corruption Canary was 0x%x, will be 0x%x. This block is guaranteed to be new yet is it already assigned.", int32(Canary), int32(ShouldBe));
				}
			}
			else if (Canary != ShouldBe && Canary != ECanary::LargeUnassigned)
			{
				UE_LOGF(LogMemory, Fatal, "MallocBinnedGPU Corruption Canary was 0x%x, will be 0x%x does not have an expected value.", int32(Canary), int32(ShouldBe));
			}
		}
		Canary = ShouldBe;
	}
	uint32 GetOSRequestedBytes() const
	{
		return AllocSize;
	}

	uint32 GetOsVMSizeDivVirtualSizeAlignment() const
	{
		CheckCanary(ECanary::LargeAssigned);
		return VMSizeDivVirtualSizeAlignment;
	}

	UPTRINT GetOsCommittedBytes() const
	{
		CheckCanary(ECanary::LargeAssigned);
		return (UPTRINT)OSCommitSize;
	}

	void SetOSAllocationSizes(uint32 InRequestedBytes, UPTRINT InCommittedBytes, uint32 InVMSizeDivVirtualSizeAlignment)
	{
		CheckCanary(ECanary::LargeAssigned);
		check(InRequestedBytes != 0);                // Shouldn't be pooling zero byte allocations
		check(InCommittedBytes >= InRequestedBytes); // We must be allocating at least as much as we requested

		AllocSize = InRequestedBytes;
		OSCommitSize = InCommittedBytes;
		VMSizeDivVirtualSizeAlignment = InVMSizeDivVirtualSizeAlignment;
	}
};


/** Hash table struct for retrieving allocation book keeping information */
struct FMallocBinnedGPU::PoolHashBucket
{
	UPTRINT BucketIndex;
	FPoolInfoLarge* FirstPool;
	PoolHashBucket* Prev;
	PoolHashBucket* Next;

	PoolHashBucket()
	{
		BucketIndex = 0;
		FirstPool = nullptr;
		Prev = this;
		Next = this;
	}

	void Link(PoolHashBucket* After)
	{
		After->Prev = Prev;
		After->Next = this;
		Prev->Next = After;
		this->Prev = After;
	}

	void Unlink()
	{
		Next->Prev = Prev;
		Prev->Next = Next;
		Prev = this;
		Next = this;
	}
};



struct FMallocBinnedGPU::Private
{
	// Implementation. 
	static CA_NO_RETURN void OutOfMemory(uint64 Size, uint32 Alignment = 0)
	{
		// this is expected not to return
		FPlatformMemory::OnOutOfMemory(Size, Alignment);
	}

	/**
	* Gets the FPoolInfoSmall for a small block memory address. If no valid info exists one is created.
	*/
	static FPoolInfoSmall* GetOrCreatePoolInfoSmall(FMallocBinnedGPU& Allocator, uint32 InPoolIndex, uint32 BlockOfBlocksIndex)
	{
		const uint32 InfosPerPage = Allocator.SmallPoolInfosPerPlatformPage;
		const uint32 InfoOuterIndex = BlockOfBlocksIndex / InfosPerPage;
		const uint32 InfoInnerIndex = BlockOfBlocksIndex % InfosPerPage;
		FPoolInfoSmall*& InfoBlock = Allocator.SmallPoolTables[InPoolIndex].PoolInfos[InfoOuterIndex];
		if (!InfoBlock)
		{
			InfoBlock = (FPoolInfoSmall*)FMemory::Malloc(Allocator.ArenaParams.BasePageSize);
			Allocator.MallocedPointers.Add(InfoBlock);
			MBG_STAT(Allocator.BinnedGPUPoolInfoMemory += Allocator.ArenaParams.AllocationGranularity;)
			DefaultConstructItems<FPoolInfoSmall>((void*)InfoBlock, InfosPerPage);
		}

		FPoolInfoSmall* Result = &InfoBlock[InfoInnerIndex];

		bool bGuaranteedToBeNew = false;
		if (BlockOfBlocksIndex >= Allocator.SmallPoolTables[InPoolIndex].NumEverUsedBlockOfBlocks)
		{
			bGuaranteedToBeNew = true;
			Allocator.SmallPoolTables[InPoolIndex].NumEverUsedBlockOfBlocks = BlockOfBlocksIndex + 1;
		}
		Result->SetCanary(FPoolInfoSmall::ECanary::SmallAssigned, false, bGuaranteedToBeNew);
		return Result;
	}
	/**
	 * Gets the FPoolInfoLarge for a large block memory address. If no valid info exists one is created.
	 */
	static FPoolInfoLarge* GetOrCreatePoolInfoLarge(FMallocBinnedGPU& Allocator, void* InPtr)
	{
		/**
		 * Creates an array of FPoolInfo structures for tracking allocations.
		 */
		auto CreatePoolArray = [](FMallocBinnedGPU& LocalAllocator)
		{
			uint64 PoolArraySize = LocalAllocator.NumLargePoolsPerPage * sizeof(FPoolInfoLarge);

			void* Result;
			Result = FMemory::Malloc(PoolArraySize);
			LocalAllocator.MallocedPointers.Add(Result);
			MBG_STAT(LocalAllocator.BinnedGPUPoolInfoMemory += PoolArraySize;)

			DefaultConstructItems<FPoolInfoLarge>(Result, LocalAllocator.NumLargePoolsPerPage);
			return (FPoolInfoLarge*)Result;
		};

		uint32 BucketIndex;
		UPTRINT BucketIndexCollision;
		uint32  PoolIndex;
		Allocator.PtrToPoolMapping.GetHashBucketAndPoolIndices(InPtr, BucketIndex, BucketIndexCollision, PoolIndex);

		PoolHashBucket* FirstBucket = &Allocator.HashBuckets[BucketIndex];
		PoolHashBucket* Collision = FirstBucket;
		do
		{
			if (!Collision->FirstPool)
			{
				Collision->BucketIndex = BucketIndexCollision;
				Collision->FirstPool = CreatePoolArray(Allocator);
				Collision->FirstPool[PoolIndex].SetCanary(FPoolInfoLarge::ECanary::LargeAssigned, false, true);
				return &Collision->FirstPool[PoolIndex];
			}

			if (Collision->BucketIndex == BucketIndexCollision)
			{
				Collision->FirstPool[PoolIndex].SetCanary(FPoolInfoLarge::ECanary::LargeAssigned, false, false);
				return &Collision->FirstPool[PoolIndex];
			}

			Collision = Collision->Next;
		} while (Collision != FirstBucket);

		// Create a new hash bucket entry
		if (!Allocator.HashBucketFreeList)
		{
			{
				Allocator.HashBucketFreeList = (PoolHashBucket*)FMemory::Malloc(Allocator.ArenaParams.AllocationGranularity);
				Allocator.MallocedPointers.Add(Allocator.HashBucketFreeList);
				MBG_STAT(Allocator.BinnedGPUHashMemory += Allocator.ArenaParams.AllocationGranularity;)
			}

			for (UPTRINT i = 0, n = Allocator.ArenaParams.AllocationGranularity / sizeof(PoolHashBucket); i < n; ++i)
			{
				Allocator.HashBucketFreeList->Link(new (Allocator.HashBucketFreeList + i) PoolHashBucket());
			}
		}

		PoolHashBucket* NextFree = Allocator.HashBucketFreeList->Next;
		PoolHashBucket* NewBucket = Allocator.HashBucketFreeList;

		NewBucket->Unlink();

		if (NextFree == NewBucket)
		{
			NextFree = nullptr;
		}
		Allocator.HashBucketFreeList = NextFree;

		if (!NewBucket->FirstPool)
		{
			NewBucket->FirstPool = CreatePoolArray(Allocator);
			NewBucket->FirstPool[PoolIndex].SetCanary(FPoolInfoLarge::ECanary::LargeAssigned, false, true);
		}
		else
		{
			NewBucket->FirstPool[PoolIndex].SetCanary(FPoolInfoLarge::ECanary::LargeAssigned, false, false);
		}

		NewBucket->BucketIndex = BucketIndexCollision;

		FirstBucket->Link(NewBucket);

		return &NewBucket->FirstPool[PoolIndex];
	}

	static FPoolInfoLarge* FindPoolInfo(FMallocBinnedGPU& Allocator, void* InPtr)
	{
		uint32 BucketIndex;
		UPTRINT BucketIndexCollision;
		uint32  PoolIndex;
		Allocator.PtrToPoolMapping.GetHashBucketAndPoolIndices(InPtr, BucketIndex, BucketIndexCollision, PoolIndex);


		PoolHashBucket* FirstBucket = &Allocator.HashBuckets[BucketIndex];
		PoolHashBucket* Collision = FirstBucket;
		do
		{
			if (Collision->BucketIndex == BucketIndexCollision)
			{
				return &Collision->FirstPool[PoolIndex];
			}

			Collision = Collision->Next;
		} while (Collision != FirstBucket);

		return nullptr;
	}


	static void FreeBundles(FMallocBinnedGPU& Allocator, FBundleNode* BundlesToRecycle, uint32 InBlockSize, uint32 InPoolIndex)
	{
		FPoolTable& Table = Allocator.SmallPoolTables[InPoolIndex];

		FBundleNode* Bundle = BundlesToRecycle;
		while (Bundle)
		{
			FBundleNode* NextBundle = Bundle->NextBundle;

			FBundleNode* Node = Bundle;
			do
			{
				FBundleNode* NextNode = Node->NextNodeInCurrentBundle;

				uint32 OutBlockOfBlocksIndex;

				void *GPUNode = ((FGPUMemoryBlockProxy*)Node)->GPUMemory;
				check(GPUNode);
				void* BasePtrOfNode = Allocator.BlockOfBlocksPointerFromContainedPtr(GPUNode, Allocator.SmallPoolTables[InPoolIndex].PagesPlatformForBlockOfBlocks, OutBlockOfBlocksIndex);
				uint32 BlockWithinIndex = (((uint8*)GPUNode) - ((uint8*)BasePtrOfNode)) / Allocator.SmallPoolTables[InPoolIndex].BlockSize;

				FPoolInfoSmall* NodePoolBlock = Allocator.SmallPoolTables[InPoolIndex].PoolInfos[OutBlockOfBlocksIndex / Allocator.SmallPoolInfosPerPlatformPage];
				if (!NodePoolBlock)
				{
					UE_LOGF(LogMemory, Fatal, "FMallocBinnedGPU Attempt to free an unrecognized small block %p", GPUNode);
				}
				FPoolInfoSmall* NodePool = &NodePoolBlock[OutBlockOfBlocksIndex % Allocator.SmallPoolInfosPerPlatformPage];

				NodePool->CheckCanary(FPoolInfoSmall::ECanary::SmallAssigned);

				bool bWasExhaused = !NodePool->FirstFreeProxy;

				// Free a pooled allocation.
				FFreeBlock* Free = (FFreeBlock*)Node;
				Free->NumFreeBlocks = 1;
				Free->NextFreeBlock = NodePool->FirstFreeProxy;
				Free->BlockSizeShifted = (InBlockSize >> Allocator.ArenaParams.MinimumAlignmentShift);
				Free->Canary = FFreeBlock::CANARY_VALUE;
				Free->PoolIndex = InPoolIndex;
				NodePool->FirstFreeProxy = Free;

				// Free this pool.
				check(NodePool->Taken >= 1);
				if (--NodePool->Taken == 0)
				{
					NodePool->SetCanary(FPoolInfoSmall::ECanary::SmallUnassigned, true, false);
					Table.BlockOfBlockAllocationBits.FreeBit(OutBlockOfBlocksIndex);

					uint64 AllocSize = static_cast<uint64>(Allocator.SmallPoolTables[InPoolIndex].PagesPlatformForBlockOfBlocks) * Allocator.ArenaParams.AllocationGranularity;

					if (!bWasExhaused)
					{
						Table.BlockOfBlockIsExhausted.AllocBit(OutBlockOfBlocksIndex);
					}

					Allocator.Decommit(InPoolIndex, BasePtrOfNode, AllocSize);
					MBG_STAT(Allocator.BinnedGPUAllocatedOSSmallPoolMemory -= AllocSize;)

					// this can be quite slow and it will sort of randomly happen when a whole page becomes free
					int64 TotalFree = 0;
					do 
					{
						FFreeBlock* NextFree = Free->NextFreeBlock;
						delete (FGPUMemoryBlockProxy*)Free;
						TotalFree += sizeof(FGPUMemoryBlockProxy);
						Free = NextFree;
					} while (Free);
					MBG_STAT(Allocator.GPUProxyMemory -= TotalFree;)
					NodePool->FirstFreeProxy = nullptr;
				}
				else if (bWasExhaused)
				{
					Table.BlockOfBlockIsExhausted.FreeBit(OutBlockOfBlocksIndex);
				}

				Node = NextNode;
			} while (Node);

			Bundle = NextBundle;
		}
	}

};


FMallocBinnedGPU::FPoolInfoSmall* FMallocBinnedGPU::PushNewPoolToFront(FMallocBinnedGPU& Allocator, uint32 InBlockSize, uint32 InPoolIndex, uint32& OutBlockOfBlocksIndex)
{
	FMallocBinnedGPU::FPoolTable& Table = Allocator.SmallPoolTables[InPoolIndex];
	const uint32 BlockOfBlocksSize = Allocator.ArenaParams.AllocationGranularity * Table.PagesPlatformForBlockOfBlocks;

	// Allocate memory.

	uint32 BlockOfBlocksIndex = Table.BlockOfBlockAllocationBits.AllocBit();
	if (BlockOfBlocksIndex == MAX_uint32)
	{
		Private::OutOfMemory(InBlockSize + 1); // The + 1 will hopefully be a hint that we actually ran out of our 1GB space.
	}
	uint8* FreePtr = BlockPointerFromIndecies(InPoolIndex, BlockOfBlocksIndex, BlockOfBlocksSize);

	Allocator.Commit(InPoolIndex, FreePtr, BlockOfBlocksSize);
	uint64 EndOffset = UPTRINT(FreePtr + BlockOfBlocksSize) - UPTRINT(PoolBasePtr(InPoolIndex));
	if (EndOffset > Table.UnusedAreaOffsetLow)
	{
		Table.UnusedAreaOffsetLow = EndOffset;
	}

	FGPUMemoryBlockProxy* Proxy = new FGPUMemoryBlockProxy(FreePtr);
	MBG_STAT(Allocator.GPUProxyMemory += sizeof(FGPUMemoryBlockProxy);)


	FFreeBlock* Free = new ((void*)Proxy) FFreeBlock(BlockOfBlocksSize, InBlockSize, InPoolIndex, Allocator.ArenaParams.MinimumAlignmentShift);


	MBG_STAT(BinnedGPUAllocatedOSSmallPoolMemory += (int64)BlockOfBlocksSize;)
	check(IsAligned(FreePtr, Allocator.ArenaParams.AllocationGranularity));
	// Create pool
	FPoolInfoSmall* Result = Private::GetOrCreatePoolInfoSmall(*this, InPoolIndex, BlockOfBlocksIndex);
	Result->CheckCanary(FPoolInfoSmall::ECanary::SmallAssigned);
	Result->Taken = 0;
	Result->FirstFreeProxy = Free;
	Table.BlockOfBlockIsExhausted.FreeBit(BlockOfBlocksIndex);

	OutBlockOfBlocksIndex = BlockOfBlocksIndex;

	return Result;
}

FMallocBinnedGPU::FPoolInfoSmall* FMallocBinnedGPU::GetFrontPool(FPoolTable& Table, uint32 InPoolIndex, uint32& OutBlockOfBlocksIndex)
{
	OutBlockOfBlocksIndex = Table.BlockOfBlockIsExhausted.NextAllocBit();
	if (OutBlockOfBlocksIndex == MAX_uint32)
	{
		return nullptr;
	}
	return Private::GetOrCreatePoolInfoSmall(*this, InPoolIndex, OutBlockOfBlocksIndex);
}


FMallocBinnedGPU::FMallocBinnedGPU()
	: HashBucketFreeList(nullptr)
{
	MBG_STAT(ConsolidatedMemory = 0;)
	MBG_STAT(GPUProxyMemory = 0;)

	ArenaParams.BasePageSize = 4096;
	ArenaParams.AllocationGranularity = FPlatformMemory::FPlatformVirtualMemoryBlock::GetCommitAlignment();

	ArenaParams.ReserveVM = [](SIZE_T Size) -> FPlatformMemory::FPlatformVirtualMemoryBlock
	{
		return FPlatformMemory::FPlatformVirtualMemoryBlock::AllocateVirtual(Size, FPlatformMemory::FPlatformVirtualMemoryBlock::GetCommitAlignment());
	};

	ArenaParams.LargeBlockAlloc = [](SIZE_T Size, SIZE_T Alignment, SIZE_T& OutCommitSize, uint32& OutVMSizeDivVirtualSizeAlignment) -> void*
	{
		FPlatformMemory::FPlatformVirtualMemoryBlock Block = FPlatformMemory::FPlatformVirtualMemoryBlock::AllocateVirtual(Size, Alignment);
		check(IsAligned(Block.GetVirtualPointer(), Alignment)); 
		OutCommitSize = Align(Size, FPlatformMemory::FPlatformVirtualMemoryBlock::GetCommitAlignment());
		Block.Commit(0, OutCommitSize);
		OutVMSizeDivVirtualSizeAlignment = Block.GetActualSizeInPages();
		return Block.GetVirtualPointer();
	};
	ArenaParams.LargeBlockFree = [](void* Ptr, uint32 VMSizeDivVirtualSizeAlignment)
	{
		FPlatformMemory::FPlatformVirtualMemoryBlock Block(Ptr, VMSizeDivVirtualSizeAlignment);
		Block.FreeVirtual();
	};

}

void FMallocBinnedGPU::InitMallocBinned()
{
	ArenaParams.MinimumAlignment = uint32(1) << ArenaParams.MinimumAlignmentShift;
	check(sizeof(FBundleNode) <= ArenaParams.MinimumAlignment);

	ArenaParams.MaxMemoryPerBlockSize = uint64(1) << ArenaParams.MaxMemoryPerBlockSizeShift;

	check(ArenaParams.BasePageSize % sizeof(FPoolInfoLarge) == 0);  // these need to divide evenly!
	NumLargePoolsPerPage = ArenaParams.BasePageSize / sizeof(FPoolInfoLarge);
	PtrToPoolMapping.Init(ArenaParams.BasePageSize, NumLargePoolsPerPage, ArenaParams.AddressLimit);

	checkf(FMath::IsPowerOfTwo(ArenaParams.AllocationGranularity), TEXT("OS page size must be a power of two"));
	checkf(FMath::IsPowerOfTwo(ArenaParams.BasePageSize), TEXT("OS page size must be a power of two"));
	check(ArenaParams.PoolCount <= 256);

	// Init pool tables.

	TArray<FSizeTableEntry> SizeTable;

	if (ArenaParams.bUseStandardSmallPoolSizes)
	{
		SizeTable.AddZeroed(UE_MBC_NUM_LISTED_SMALL_POOLS + ArenaParams.MaxStandardPoolSize / ArenaParams.BasePageSize); // overestimate
		ArenaParams.PoolCount = FSizeTableEntry::FillSizeTable(ArenaParams.AllocationGranularity, &SizeTable[0], ArenaParams.BasePageSize, ArenaParams.MaxStandardPoolSize, ArenaParams.BasePageSize);
		SizeTable.RemoveAt(ArenaParams.PoolCount, SizeTable.Num() - ArenaParams.PoolCount);
	}
	else
	{
		ArenaParams.PoolCount = 0;
	}
	for (uint32 Size : ArenaParams.AdditionalBlockSizes)
	{
		if (Size > ArenaParams.BasePageSize)
		{
			check(Size % 4096 == 0); // calculations are done assume 4k is the smallest page size we will ever see
			check(Size / 4096 <= std::numeric_limits<uint8>::max())		// Make sure we don't try to allocate more pages than fits in our counter.
			SizeTable.Emplace(Size, ArenaParams.AllocationGranularity, (uint8)(Size / 4096), ArenaParams.BasePageSize);
		}
		else
		{
			// it is difficult to test what would actually make a good bucket size here, wouldn't want a prime number, 33 for example because that would take 33 pages a slab
			SizeTable.Emplace(Size, ArenaParams.AllocationGranularity, (uint8)1, ArenaParams.BasePageSize);
		}
		ArenaParams.PoolCount++;
	}
	if (ArenaParams.AdditionalBlockSizes.Num())
	{
		Algo::Sort(SizeTable);
	}
	check(ArenaParams.PoolCount == SizeTable.Num());
	check(SizeTable.Num() < 256);
	ArenaParams.MaxPoolSize = SizeTable[ArenaParams.PoolCount - 1].BinSize;

	check(ArenaParams.BasePageSize % sizeof(FPoolInfoSmall) == 0);
	SmallPoolInfosPerPlatformPage = ArenaParams.BasePageSize / sizeof(FPoolInfoSmall);

	GGlobalRecycler.Init(ArenaParams.PoolCount);
	SmallPoolTables.AddDefaulted(ArenaParams.PoolCount);
	SmallBlockSizesReversedShifted.AddDefaulted(ArenaParams.PoolCount);
	PoolBaseVMPtr.AddDefaulted(ArenaParams.PoolCount);
	PoolBaseVMBlocks.AddDefaulted(ArenaParams.PoolCount);
	MemSizeToIndex.AddDefaulted(1 + (ArenaParams.MaxPoolSize >> ArenaParams.MinimumAlignmentShift));

	ArenaParams.MaximumAlignmentForSmallBlock = ArenaParams.MinimumAlignment;
	check(ArenaParams.MaximumAlignmentForSmallBlock > 0);

	for (uint32 Index = 0; Index < ArenaParams.PoolCount; ++Index)
	{
		checkf(Index == 0 || SizeTable[Index - 1].BinSize < SizeTable[Index].BinSize, TEXT("Small block sizes must be strictly increasing"));
		checkf(SizeTable[Index].BinSize % ArenaParams.MinimumAlignment == 0, TEXT("Small block size must be a multiple of ArenaParams.MinimumAlignment"));

		// determine the largest alignment that we can cover with a small block
		while (ArenaParams.MaximumAlignmentForSmallBlock < ArenaParams.AllocationGranularity && IsAligned(SizeTable[Index].BinSize, ArenaParams.MaximumAlignmentForSmallBlock * 2))
		{
			ArenaParams.MaximumAlignmentForSmallBlock *= 2;
		}

		SmallPoolTables[Index].BlockSize = SizeTable[Index].BinSize;
		SmallPoolTables[Index].PagesPlatformForBlockOfBlocks = SizeTable[Index].NumMemoryPagesPerBlock;

		SmallPoolTables[Index].UnusedAreaOffsetLow = 0;
		SmallPoolTables[Index].NumEverUsedBlockOfBlocks = 0;

		int64 TotalNumberOfBlocksOfBlocks = ArenaParams.MaxMemoryPerBlockSize / (SizeTable[Index].NumMemoryPagesPerBlock * ArenaParams.AllocationGranularity);

		int64 MaxPoolInfoMemory = sizeof(FPoolInfoSmall**) * (TotalNumberOfBlocksOfBlocks + SmallPoolInfosPerPlatformPage - 1) / SmallPoolInfosPerPlatformPage;
		SmallPoolTables[Index].PoolInfos = (FPoolInfoSmall**)FMemory::Malloc(MaxPoolInfoMemory);
		MallocedPointers.Add(SmallPoolTables[Index].PoolInfos);

		FMemory::Memzero(SmallPoolTables[Index].PoolInfos, MaxPoolInfoMemory);
		MBG_STAT(BinnedGPUPoolInfoMemory += MaxPoolInfoMemory;)

		{
			int64 AllocationSize = FBitTree::GetMemoryRequirements(TotalNumberOfBlocksOfBlocks);

			{
				void *Bits = FMemory::Malloc(AllocationSize);
				MallocedPointers.Add(Bits);
				check(Bits);
				MBG_STAT(BinnedGPUFreeBitsMemory += AllocationSize;)
				SmallPoolTables[Index].BlockOfBlockAllocationBits.FBitTreeInit(TotalNumberOfBlocksOfBlocks, Bits, AllocationSize, false);
			}
			{
				void *Bits = FMemory::Malloc(AllocationSize);
				MallocedPointers.Add(Bits);
				check(Bits);
				MBG_STAT(BinnedGPUFreeBitsMemory += AllocationSize;)
				SmallPoolTables[Index].BlockOfBlockIsExhausted.FBitTreeInit(TotalNumberOfBlocksOfBlocks, Bits, AllocationSize, true);
			}
		}
	}


	// Set up pool mappings
	uint8* IndexEntry = &MemSizeToIndex[0];
	uint32  PoolIndex = 0;
	for (uint32 Index = 0; Index != 1 + (ArenaParams.MaxPoolSize >> ArenaParams.MinimumAlignmentShift); ++Index)
	{

		uint32 BlockSize = Index << ArenaParams.MinimumAlignmentShift; // inverse of int32 Index = int32((Size >> ArenaParams.MinimumAlignmentShift));
		while (SizeTable[PoolIndex].BinSize < BlockSize)
		{
			++PoolIndex;
			check(PoolIndex != ArenaParams.PoolCount);
		}
		check(PoolIndex < 256);
		*IndexEntry++ = uint8(PoolIndex);
	}
	// now reverse the pool sizes for cache coherency

	for (uint32 Index = 0; Index != ArenaParams.PoolCount; ++Index)
	{
		uint32 Partner = ArenaParams.PoolCount - Index - 1;
		SmallBlockSizesReversedShifted[Index] = (SizeTable[Partner].BinSize >> ArenaParams.MinimumAlignmentShift);
	}
	uint64 MaxHashBuckets = PtrToPoolMapping.GetMaxHashBuckets();

	{
		int64 HashAllocSize = MaxHashBuckets * sizeof(PoolHashBucket);
		HashBuckets = (PoolHashBucket*)FMemory::Malloc(HashAllocSize);
		MallocedPointers.Add(HashBuckets);

		MBG_STAT(BinnedGPUHashMemory += HashAllocSize;)
		verify(HashBuckets);
	}

	DefaultConstructItems<PoolHashBucket>(HashBuckets, MaxHashBuckets);

	uint8* BinnedGPUBaseVMPtr;
	if (!ArenaParams.bUseSeparateVMPerPool)
	{
		PoolBaseVMBlock = ArenaParams.ReserveVM(ArenaParams.PoolCount * ArenaParams.MaxMemoryPerBlockSize);
		BinnedGPUBaseVMPtr = (uint8*)PoolBaseVMBlock.GetVirtualPointer();
	}
	else
	{
		BinnedGPUBaseVMPtr = nullptr;
	}
	for (uint32 Index = 0; Index < ArenaParams.PoolCount; ++Index)
	{
		uint8* NewVM;
		FPlatformMemory::FPlatformVirtualMemoryBlock NewBlock;
		if (BinnedGPUBaseVMPtr)
		{
			NewVM = BinnedGPUBaseVMPtr;
			BinnedGPUBaseVMPtr += ArenaParams.MaxMemoryPerBlockSize;
		}
		else
		{
			NewBlock = ArenaParams.ReserveVM(ArenaParams.MaxMemoryPerBlockSize);
			NewVM = (uint8*)NewBlock.GetVirtualPointer();
		}

		// insertion sort
		if (Index && NewVM < PoolBaseVMPtr[Index - 1])
		{
			uint32 InsertIndex = 0;
			for (; InsertIndex < Index; ++InsertIndex)
			{
				if (NewVM < PoolBaseVMPtr[InsertIndex])
				{
					break;
				}
			}
			check(InsertIndex < Index);
			for (uint32 MoveIndex = Index; MoveIndex > InsertIndex; --MoveIndex)
			{
				PoolBaseVMPtr[MoveIndex] = PoolBaseVMPtr[MoveIndex - 1];
				PoolBaseVMBlocks[MoveIndex] = PoolBaseVMBlocks[MoveIndex - 1];
			}
			PoolBaseVMPtr[InsertIndex] = NewVM;
			PoolBaseVMBlocks[InsertIndex] = NewBlock;
		}
		else
		{
			PoolBaseVMPtr[Index] = NewVM;
			PoolBaseVMBlocks[Index] = NewBlock;
		}
	}
	HighestPoolBaseVMPtr = PoolBaseVMPtr[ArenaParams.PoolCount - 1];
	uint64 TotalGaps = 0;
	for (uint32 Index = 0; Index < uint32(ArenaParams.PoolCount - 1); ++Index)
	{
		check(PoolBaseVMPtr[Index + 1] > PoolBaseVMPtr[Index]); // we sorted it
		check(PoolBaseVMPtr[Index + 1] >= PoolBaseVMPtr[Index] + ArenaParams.MaxMemoryPerBlockSize); // and blocks are non-overlapping
		TotalGaps += PoolBaseVMPtr[Index + 1] - (PoolBaseVMPtr[Index] + ArenaParams.MaxMemoryPerBlockSize);
	}
	if (TotalGaps == 0)
	{
		PoolSearchDiv = 0;
	}
	else if (TotalGaps < ArenaParams.MaxMemoryPerBlockSize)
	{
		check(ArenaParams.bUseSeparateVMPerPool);
		PoolSearchDiv = ArenaParams.MaxMemoryPerBlockSize; // the gaps are not significant, ignoring them should give accurate searches
	}
	else
	{
		check(ArenaParams.bUseSeparateVMPerPool);
		PoolSearchDiv = ArenaParams.MaxMemoryPerBlockSize + ((TotalGaps + ArenaParams.PoolCount - 2) / (ArenaParams.PoolCount - 1));
	}
}

void FMallocBinnedGPU::Commit(uint32 InPoolIndex, void *Ptr, SIZE_T Size)
{
	if (!ArenaParams.bUseSeparateVMPerPool)
	{
		PoolBaseVMBlock.CommitByPtr(Ptr, Size);
	}
	else
	{
		PoolBaseVMBlocks[InPoolIndex].CommitByPtr(Ptr, Size);
	}
}
void FMallocBinnedGPU::Decommit(uint32 InPoolIndex, void *Ptr, SIZE_T Size)
{
	if (!ArenaParams.bUseSeparateVMPerPool)
	{
		PoolBaseVMBlock.DecommitByPtr(Ptr, Size);
	}
	else
	{
		PoolBaseVMBlocks[InPoolIndex].DecommitByPtr(Ptr, Size);
	}
}


FMallocBinnedGPU::~FMallocBinnedGPU()
{
	UE::TUniqueLock Lock(Mutex);
	UE::TUniqueLock Lock2(GetFreeBlockListsRegistrationMutex());

	MBG_STAT(
		UE_CLOGF(BinnedGPUAllocatedOSSmallPoolMemory > 0, LogCore, Error, "FMallocBinnedGPU leaked small block memory: %fmb", ((double)BinnedGPUAllocatedOSSmallPoolMemory) / (1024.0f * 1024.0f));
		UE_CLOGF(BinnedGPUAllocatedLargePoolMemoryWAlignment > 0, LogCore, Error, "FMallocBinnedGPU leaked large block memory: %fmb", ((double)BinnedGPUAllocatedLargePoolMemoryWAlignment) / (1024.0f * 1024.0f));
	)
	for (FPerThreadFreeBlockLists* Lists : GetRegisteredFreeBlockLists())
	{
		if (Lists)
		{
			for (int32 PoolIndex = 0; PoolIndex != ArenaParams.PoolCount; ++PoolIndex)
			{
				FBundleNode* Bundles = Lists->PopBundles(PoolIndex);
				if (Bundles)
				{
					Private::FreeBundles(*this, Bundles, PoolIndexToBlockSize(PoolIndex), PoolIndex);
				}
			}
			delete Lists;
		}
	}

	if (ArenaParams.bUseSeparateVMPerPool)
	{
		for (int32 PoolIndex = 0; PoolIndex != ArenaParams.PoolCount; ++PoolIndex)
		{
			PoolBaseVMBlocks[PoolIndex].FreeVirtual();
		}
	}
	else
	{
		PoolBaseVMBlock.FreeVirtual();
	}


	for (void* Ptr : MallocedPointers)
	{
		FMemory::Free(Ptr);
	}

	FPlatformTLS::FreeTlsSlot(BinnedGPUTlsSlot);
}

bool FMallocBinnedGPU::IsInternallyThreadSafe() const
{
	return true;
}

void* FMallocBinnedGPU::MallocExternal(SIZE_T Size, uint32 Alignment)
{
	check(FMath::IsPowerOfTwo(Alignment));
	checkf(DEFAULT_ALIGNMENT <= ArenaParams.MinimumAlignment, TEXT("DEFAULT_ALIGNMENT is assumed to be zero")); //-V547

	if (AdjustSmallBlockSizeForAlignment(Size, Alignment)) // there is some redundant work here... we already adjusted the size for alignment
	{
		uint32 PoolIndex = BoundSizeToPoolIndex(Size);
		FPerThreadFreeBlockLists* Lists = ArenaParams.bPerThreadCaches ? FPerThreadFreeBlockLists::Get(BinnedGPUTlsSlot) : nullptr;
		if (Lists)
		{
			if (Lists->ObtainRecycledPartial(ArenaParams, GGlobalRecycler, PoolIndex))
			{
				if (void* Result = Lists->Malloc(*this, PoolIndex))
				{
					uint32 BlockSize = PoolIndexToBlockSize(PoolIndex);
					Lists->AllocatedMemory += BlockSize;
					checkSlow(IsAligned(Result, Alignment));
					return Result;
				}
			}
		}

		UE::TUniqueLock Lock(Mutex);

		// Allocate from small object pool.
		FPoolTable& Table = SmallPoolTables[PoolIndex];

		uint32 BlockOfBlocksIndex = MAX_uint32;
		FPoolInfoSmall* Pool = GetFrontPool(Table, PoolIndex, BlockOfBlocksIndex);
		if (!Pool)
		{
			Pool = PushNewPoolToFront(*this, Table.BlockSize, PoolIndex, BlockOfBlocksIndex);
		}

		const uint32 BlockOfBlocksSize = ArenaParams.AllocationGranularity * Table.PagesPlatformForBlockOfBlocks;
		uint8* BlockOfBlocksPtr = BlockPointerFromIndecies(PoolIndex, BlockOfBlocksIndex, BlockOfBlocksSize);

		void* Result = Pool->AllocateRegularBlock(*this, ArenaParams.MinimumAlignmentShift);
		MBG_STAT(BinnedGPUAllocatedSmallPoolMemory += PoolIndexToBlockSize(PoolIndex);)
		if (ArenaParams.EmptyCacheAllocExtra)
		{
			if (Lists)
			{
				// prefill the free list with some allocations so we are less likely to hit this slow path with the mutex 
				for (int32 Index = 0; Index < ArenaParams.EmptyCacheAllocExtra && Pool->HasFreeRegularBlock(); Index++)
				{
					if (!Lists->Free(*this, Result, PoolIndex, Table.BlockSize, ArenaParams))
					{
						break;
					}
					Result = Pool->AllocateRegularBlock(*this, ArenaParams.MinimumAlignmentShift);
				}
			}
		}
		if (!Pool->HasFreeRegularBlock())
		{
			Table.BlockOfBlockIsExhausted.AllocBit(BlockOfBlocksIndex);
		}
		checkSlow(IsAligned(Result, Alignment));
		return Result;
	}

	uint32 VMSizeDivVirtualSizeAlignment = 0;
	SIZE_T CommitSize = 0;
	void* Result = ArenaParams.LargeBlockAlloc(Size, Alignment, CommitSize, VMSizeDivVirtualSizeAlignment);

	UE_CLOGF(!IsAligned(Result, Alignment), LogMemory, Fatal, "FMallocBinnedGPU alignment was too large for OS. Alignment=%d   Ptr=%p", Alignment, Result);

	if (!Result)
	{
		Private::OutOfMemory(Size);
	}
	check(IsOSAllocation(Result));
	UE::TUniqueLock Lock(Mutex);

	MBG_STAT(BinnedGPUAllocatedLargePoolMemory += (int64)Size;)
	MBG_STAT(BinnedGPUAllocatedLargePoolMemoryWAlignment += CommitSize;)

	// Create pool.
	FPoolInfoLarge* Pool = Private::GetOrCreatePoolInfoLarge(*this, Result);
	check(Size > 0 && Size <= CommitSize && CommitSize >= ArenaParams.BasePageSize);
	Pool->SetOSAllocationSizes(Size, CommitSize, VMSizeDivVirtualSizeAlignment);

	return Result;
}



void FMallocBinnedGPU::FreeExternal(void* Ptr)
{
	uint64 PoolIndex = PoolIndexFromPtr(Ptr);
	if (PoolIndex < ArenaParams.PoolCount)
	{
		check(Ptr); // null is an OS allocation because it will not fall in our VM block
		uint32 BlockSize = PoolIndexToBlockSize(PoolIndex);

		FBundleNode* BundlesToRecycle = nullptr;
		FPerThreadFreeBlockLists* Lists = ArenaParams.bPerThreadCaches ? FPerThreadFreeBlockLists::Get(BinnedGPUTlsSlot) : nullptr;
		if (Lists)
		{
			BundlesToRecycle = Lists->RecycleFullBundle(ArenaParams, GGlobalRecycler, PoolIndex);
			bool bPushed = Lists->Free(*this, Ptr, PoolIndex, BlockSize, ArenaParams);
			check(bPushed);
			Lists->AllocatedMemory -= BlockSize;
		}
		else
		{
			BundlesToRecycle = (FBundleNode*)new FGPUMemoryBlockProxy(Ptr);
			MBG_STAT(GPUProxyMemory += sizeof(FGPUMemoryBlockProxy);)
			BundlesToRecycle->NextNodeInCurrentBundle = nullptr;
		}
		if (BundlesToRecycle)
		{
			BundlesToRecycle->NextBundle = nullptr;
			UE::TUniqueLock Lock(Mutex);
			Private::FreeBundles(*this, BundlesToRecycle, BlockSize, PoolIndex);
			if (!Lists)
			{
				// lists track their own stat track them instead in the global stat if we don't have lists
				MBG_STAT(BinnedGPUAllocatedSmallPoolMemory -= ((int64)(BlockSize));)
			}
		}
	}
	else if (Ptr)
	{
		uint32 VMSizeDivVirtualSizeAlignment;

		{
			UE::TUniqueLock Lock(Mutex);
			FPoolInfoLarge* Pool = Private::FindPoolInfo(*this, Ptr);
			if (!Pool)
			{
				UE_LOGF(LogMemory, Fatal, "FMallocBinnedGPU Attempt to free an unrecognized block %p", Ptr);
			}
			UPTRINT PoolOsCommittedBytes = Pool->GetOsCommittedBytes();
			UPTRINT PoolOSRequestedBytes = Pool->GetOSRequestedBytes();
			VMSizeDivVirtualSizeAlignment = Pool->GetOsVMSizeDivVirtualSizeAlignment();

			MBG_STAT(BinnedGPUAllocatedLargePoolMemory -= ((int64)PoolOSRequestedBytes);)
			MBG_STAT(BinnedGPUAllocatedLargePoolMemoryWAlignment -= ((int64)PoolOsCommittedBytes);)

			checkf(PoolOSRequestedBytes <= PoolOsCommittedBytes, TEXT("FMallocBinnedGPU::FreeExternal %d %d"), int32(PoolOSRequestedBytes), int32(PoolOsCommittedBytes));
			Pool->SetCanary(FPoolInfoLarge::ECanary::LargeUnassigned, true, false);
		}
		// Free an OS allocation.
		ArenaParams.LargeBlockFree(Ptr, VMSizeDivVirtualSizeAlignment);
	}
}

bool FMallocBinnedGPU::GetAllocationSizeExternal(void* Ptr, SIZE_T& SizeOut)
{
	uint64 PoolIndex = PoolIndexFromPtr(Ptr);
	if (PoolIndex < ArenaParams.PoolCount)
	{
		check(Ptr); // null is an OS allocation because it will not fall in our VM block
		SizeOut = PoolIndexToBlockSize(PoolIndex);
		return true;
	}
	if (!Ptr)
	{
		return false;
	}
	UE::TUniqueLock Lock(Mutex);
	FPoolInfoLarge* Pool = Private::FindPoolInfo(*this, Ptr);
	if (!Pool)
	{
		UE_LOGF(LogMemory, Fatal, "FMallocBinnedGPU Attempt to GetAllocationSizeExternal an unrecognized block %p", Ptr);
	}
	UPTRINT PoolOsBytes = Pool->GetOsCommittedBytes();
	uint32 PoolOSRequestedBytes = Pool->GetOSRequestedBytes();
	checkf(PoolOSRequestedBytes <= PoolOsBytes, TEXT("FMallocBinnedGPU::GetAllocationSizeExternal %d %d"), int32(PoolOSRequestedBytes), int32(PoolOsBytes));
	SizeOut = PoolOsBytes;
	return true;
}

bool FMallocBinnedGPU::ValidateHeap()
{
	// Not implemented
	// NumEverUsedBlockOfBlocks gives us all of the information we need to examine each pool, so it is doable.
	return true;
}

const TCHAR* FMallocBinnedGPU::GetDescriptiveName()
{
	return TEXT("BinnedGPU");
}

void FMallocBinnedGPU::FlushCurrentThreadCache()
{
	double StartTimeInner = FPlatformTime::Seconds();
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FMallocBinnedGPU_FlushCurrentThreadCache);
	FPerThreadFreeBlockLists* Lists = FPerThreadFreeBlockLists::Get(BinnedGPUTlsSlot);

	double WaitForMutexTime = 0.0;
	double WaitForMutexAndTrimTime = 0.0;

	if (Lists)
	{
		UE::TUniqueLock Lock(Mutex);
		WaitForMutexTime = FPlatformTime::Seconds() - StartTimeInner;
		for (int32 PoolIndex = 0; PoolIndex != ArenaParams.PoolCount; ++PoolIndex)
		{
			FBundleNode* Bundles = Lists->PopBundles(PoolIndex);
			if (Bundles)
			{
				Private::FreeBundles(*this, Bundles, PoolIndexToBlockSize(PoolIndex), PoolIndex);
			}
		}
		WaitForMutexAndTrimTime = FPlatformTime::Seconds() - StartTimeInner;
	}

	// These logs must happen outside the above mutex to avoid deadlocks
	if (WaitForMutexTime > 0.02f)
	{
		UE_LOGF(LogMemory, Warning, "FMallocBinnedGPU took %6.2fms to wait for mutex for trim.", WaitForMutexTime * 1000.0f);
	}
	if (WaitForMutexAndTrimTime > 0.02f)
	{
		UE_LOGF(LogMemory, Warning, "FMallocBinnedGPU took %6.2fms to wait for mutex AND trim.", WaitForMutexAndTrimTime * 1000.0f);
	}
}

#include "Async/TaskGraphInterfaces.h"

void FMallocBinnedGPU::Trim(bool bTrimThreadCaches)
{
	if (bTrimThreadCaches && ArenaParams.bPerThreadCaches)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FMallocBinnedGPU_Trim);
		//double StartTime = FPlatformTime::Seconds();
		TFunction<void(ENamedThreads::Type CurrentThread)> Broadcast =
			[this](ENamedThreads::Type MyThread)
		{
			FlushCurrentThreadCache();
		};
		// Skip task threads on desktop platforms as it is too slow and they don't have much memory
		FTaskGraphInterface::BroadcastSlow_OnlyUseForSpecialPurposes(!PLATFORM_DESKTOP, false, Broadcast);
		//UE_LOGF(LogTemp, Display, "Trim Broadcast = %6.2fms", 1000.0f * float(FPlatformTime::Seconds() - StartTime));
	}
}

void FMallocBinnedGPU::SetupTLSCachesOnCurrentThread()
{
	if (!ArenaParams.bPerThreadCaches)
	{
		return;
	}
	if (!FPlatformTLS::IsValidTlsSlot(BinnedGPUTlsSlot))
	{
		BinnedGPUTlsSlot = FPlatformTLS::AllocTlsSlot();
	}
	check(FPlatformTLS::IsValidTlsSlot(BinnedGPUTlsSlot));
	FPerThreadFreeBlockLists::SetTLS(*this);
}

void FMallocBinnedGPU::ClearAndDisableTLSCachesOnCurrentThread()
{
	FlushCurrentThreadCache();
	MBG_STAT(ConsolidatedMemory +=) FPerThreadFreeBlockLists::ClearTLS(*this);
}


bool FMallocBinnedGPU::FFreeBlockList::ObtainPartial(FArenaParams& LocalArenaParams, FGlobalRecycler& GlobalRecycler, uint32 InPoolIndex)
{
	if (!PartialBundle.Head)
	{
		PartialBundle.Count = 0;
		PartialBundle.Head = GlobalRecycler.PopBundle(LocalArenaParams.MaxGlobalBundles, InPoolIndex);
		if (PartialBundle.Head)
		{
			PartialBundle.Count = PartialBundle.Head->Count;
			PartialBundle.Head->NextBundle = nullptr;
			return true;
		}
		return false;
	}
	return true;
}

FMallocBinnedGPU::FBundleNode* FMallocBinnedGPU::FFreeBlockList::RecyleFull(FArenaParams& LocalArenaParams, FGlobalRecycler& GlobalRecycler, uint32 InPoolIndex)
{
	FMallocBinnedGPU::FBundleNode* Result = nullptr;
	if (FullBundle.Head)
	{
		FullBundle.Head->Count = FullBundle.Count;
		if (!GlobalRecycler.PushBundle(LocalArenaParams.MaxGlobalBundles, InPoolIndex, FullBundle.Head))
		{
			Result = FullBundle.Head;
			Result->NextBundle = nullptr;
		}
		FullBundle.Reset();
	}
	return Result;
}

FMallocBinnedGPU::FBundleNode* FMallocBinnedGPU::FFreeBlockList::PopBundles(uint32 InPoolIndex)
{
	FBundleNode* Partial = PartialBundle.Head;
	if (Partial)
	{
		PartialBundle.Reset();
		Partial->NextBundle = nullptr;
	}

	FBundleNode* Full = FullBundle.Head;
	if (Full)
	{
		FullBundle.Reset();
		Full->NextBundle = nullptr;
	}

	FBundleNode* Result = Partial;
	if (Result)
	{
		Result->NextBundle = Full;
	}
	else
	{
		Result = Full;
	}

	return Result;
}

void FMallocBinnedGPU::FPerThreadFreeBlockLists::SetTLS(FMallocBinnedGPU& Allocator)
{
	uint32 BinnedGPUTlsSlot = Allocator.BinnedGPUTlsSlot;
	check(FPlatformTLS::IsValidTlsSlot(BinnedGPUTlsSlot));
	FPerThreadFreeBlockLists* ThreadSingleton = (FPerThreadFreeBlockLists*)FPlatformTLS::GetTlsValue(BinnedGPUTlsSlot);
	if (!ThreadSingleton)
	{
		int64 TLSSize = sizeof(FPerThreadFreeBlockLists);
		ThreadSingleton = new FPerThreadFreeBlockLists(Allocator.ArenaParams.PoolCount);
		MBG_STAT(Allocator.BinnedGPUTLSMemory += TLSSize;)
		verify(ThreadSingleton);
		FPlatformTLS::SetTlsValue(BinnedGPUTlsSlot, ThreadSingleton);
		Allocator.RegisterThreadFreeBlockLists(ThreadSingleton);
	}
}

int64 FMallocBinnedGPU::FPerThreadFreeBlockLists::ClearTLS(FMallocBinnedGPU& Allocator)
{
	uint32 BinnedGPUTlsSlot = Allocator.BinnedGPUTlsSlot;
	check(FPlatformTLS::IsValidTlsSlot(BinnedGPUTlsSlot));
	int64 Result = 0;
	FPerThreadFreeBlockLists* ThreadSingleton = (FPerThreadFreeBlockLists*)FPlatformTLS::GetTlsValue(BinnedGPUTlsSlot);
	if (ThreadSingleton)
	{
		int64 TLSSize = sizeof(FPerThreadFreeBlockLists);
		MBG_STAT(Allocator.BinnedGPUTLSMemory -= TLSSize;)
		Result = Allocator.UnregisterThreadFreeBlockLists(ThreadSingleton);
	}
	FPlatformTLS::SetTlsValue(BinnedGPUTlsSlot, nullptr);
	return Result;
}

void FMallocBinnedGPU::FFreeBlock::CanaryFail() const
{
	UE_LOGF(LogMemory, Fatal, "FMallocBinnedGPU Attempt to realloc an unrecognized block %p   canary == 0x%x != 0x%x", (void*)this, (int32)Canary, (int32)FMallocBinnedGPU::FFreeBlock::CANARY_VALUE);
}

MBG_STAT(
int64 FMallocBinnedGPU::GetTotalAllocatedSmallPoolMemory()
{
	int64 FreeBlockAllocatedMemory = 0;
	{
		UE::TUniqueLock Lock(GetFreeBlockListsRegistrationMutex());
		for (const FPerThreadFreeBlockLists* FreeBlockLists : GetRegisteredFreeBlockLists())
		{
			FreeBlockAllocatedMemory += FreeBlockLists->AllocatedMemory;
		}
		FreeBlockAllocatedMemory += ConsolidatedMemory;
	}
	return BinnedGPUAllocatedSmallPoolMemory + FreeBlockAllocatedMemory;
})

void FMallocBinnedGPU::GetAllocatorStats(FGenericMemoryStats& OutStats)
{
	MBG_STAT(
		int64 TotalAllocatedSmallPoolMemory = GetTotalAllocatedSmallPoolMemory();

		OutStats.Add(TEXT("BinnedGPUAllocatedSmallPoolMemory"), TotalAllocatedSmallPoolMemory);
		OutStats.Add(TEXT("BinnedGPUAllocatedOSSmallPoolMemory"), BinnedGPUAllocatedOSSmallPoolMemory);
		OutStats.Add(TEXT("BinnedGPUAllocatedLargePoolMemory"), BinnedGPUAllocatedLargePoolMemory);
		OutStats.Add(TEXT("BinnedGPUAllocatedLargePoolMemoryWAlignment"), BinnedGPUAllocatedLargePoolMemoryWAlignment);

		uint64 TotalAllocated = TotalAllocatedSmallPoolMemory + BinnedGPUAllocatedLargePoolMemory;
		uint64 TotalOSAllocated = BinnedGPUAllocatedOSSmallPoolMemory + BinnedGPUAllocatedLargePoolMemoryWAlignment;

		OutStats.Add(TEXT("TotalAllocated"), TotalAllocated);
		OutStats.Add(TEXT("TotalOSAllocated"), TotalOSAllocated);
	)
	FMalloc::GetAllocatorStats(OutStats);
}

void FMallocBinnedGPU::DumpAllocatorStats(class FOutputDevice& Ar)
{

	Ar.Logf(TEXT("FMallocBinnedGPU Mem report"));
	Ar.Logf(TEXT("Constants.BinnedAllocationGranularity = %d"), int32(ArenaParams.AllocationGranularity));
	Ar.Logf(TEXT("ArenaParams.MaxPoolSize = %d"), int32(ArenaParams.MaxPoolSize));
	Ar.Logf(TEXT("MAX_MEMORY_PER_BLOCK_SIZE = %llu"), uint64(ArenaParams.MaxMemoryPerBlockSize));
	MBG_STAT(
		int64 TotalAllocatedSmallPoolMemory = GetTotalAllocatedSmallPoolMemory();
		Ar.Logf(TEXT("Small Pool Allocations: %fmb  (including block size padding)"), ((double)TotalAllocatedSmallPoolMemory) / (1024.0f * 1024.0f));
		Ar.Logf(TEXT("Small Pool OS Allocated: %fmb"), ((double)BinnedGPUAllocatedOSSmallPoolMemory) / (1024.0f * 1024.0f));
		Ar.Logf(TEXT("Large Pool Requested Allocations: %fmb"), ((double)BinnedGPUAllocatedLargePoolMemory) / (1024.0f * 1024.0f));
		Ar.Logf(TEXT("Large Pool OS Allocated: %fmb"), ((double)BinnedGPUAllocatedLargePoolMemoryWAlignment) / (1024.0f * 1024.0f));
		Ar.Logf(TEXT("PoolInfo: %fmb"), ((double)BinnedGPUPoolInfoMemory) / (1024.0f * 1024.0f));
		Ar.Logf(TEXT("Hash: %fmb"), ((double)BinnedGPUHashMemory) / (1024.0f * 1024.0f));
		Ar.Logf(TEXT("Free Bits: %fmb"), ((double)BinnedGPUFreeBitsMemory) / (1024.0f * 1024.0f));
		Ar.Logf(TEXT("TLS: %fmb"), ((double)BinnedGPUTLSMemory) / (1024.0f * 1024.0f));
		Ar.Logf(TEXT("GPU Memory Proxies: %fmb"), ((double)GPUProxyMemory) / (1024.0f * 1024.0f));
		Ar.Logf(TEXT("Total allocated from OS: %fmb"),
			((double)
				BinnedGPUAllocatedOSSmallPoolMemory + BinnedGPUAllocatedLargePoolMemoryWAlignment + BinnedGPUPoolInfoMemory + BinnedGPUHashMemory + BinnedGPUFreeBitsMemory + BinnedGPUTLSMemory + GPUProxyMemory
				) / (1024.0f * 1024.0f));
	)
	Ar.Logf(TEXT("BINNEDGPU_USE_SEPARATE_VM_PER_POOL is true - VM is Contiguous = %d"), PoolSearchDiv == 0);
	if (PoolSearchDiv)
	{
		uint64 TotalMem = PoolBaseVMPtr[ArenaParams.PoolCount - 1] + ArenaParams.MaxMemoryPerBlockSize - PoolBaseVMPtr[0];
		uint64 MinimumMem = uint64(ArenaParams.PoolCount) * ArenaParams.MaxMemoryPerBlockSize;
		Ar.Logf(TEXT("Percent of gaps in the address range %6.4f  (hopefully < 1, or the searches above will suffer)"), 100.0f * (1.0f - float(MinimumMem) / float(TotalMem)));
	}

	for (int32 PoolIndex = 0; PoolIndex < ArenaParams.PoolCount; PoolIndex++)
	{

		int64 VM = SmallPoolTables[PoolIndex].UnusedAreaOffsetLow;
		uint32 CommittedBlocks = SmallPoolTables[PoolIndex].BlockOfBlockAllocationBits.CountOnes(SmallPoolTables[PoolIndex].NumEverUsedBlockOfBlocks);
		uint32 PartialBlocks = SmallPoolTables[PoolIndex].NumEverUsedBlockOfBlocks - SmallPoolTables[PoolIndex].BlockOfBlockIsExhausted.CountOnes(SmallPoolTables[PoolIndex].NumEverUsedBlockOfBlocks);
		uint32 FullBlocks = CommittedBlocks - PartialBlocks;
		int64 ComittedVM = VM - (SmallPoolTables[PoolIndex].NumEverUsedBlockOfBlocks - CommittedBlocks) * SmallPoolTables[PoolIndex].PagesPlatformForBlockOfBlocks * ArenaParams.AllocationGranularity;


		Ar.Logf(TEXT("Pool %2d   Size %6d   UsedVM %3dMB  CommittedVM %3dMB  HighSlabs %6d  CommittedSlabs %6d  FullSlabs %6d  PartialSlabs  %6d"),
			PoolIndex,
			PoolIndexToBlockSize(PoolIndex),
			VM / (1024 * 1024),
			ComittedVM / (1024 * 1024),
			SmallPoolTables[PoolIndex].NumEverUsedBlockOfBlocks,
			CommittedBlocks,
			FullBlocks,
			PartialBlocks
		);
	}
}
#endif

PRAGMA_RESTORE_UNSAFE_TYPECAST_WARNINGS
