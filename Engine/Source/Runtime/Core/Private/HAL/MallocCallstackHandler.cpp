// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	FMallocCallstackHandler.cpp: Memory tracking allocator
=============================================================================*/
#include "HAL/MallocCallstackHandler.h"
#include "Logging/LogMacros.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Misc/Parse.h"
#include "Misc/ScopeLock.h"
#include "GenericPlatform/GenericPlatformProcess.h"
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformMisc.h"
#include "HAL/PlatformStackWalk.h"

FMallocCallstackHandler* GMallocCallstackHandler = nullptr;

FMallocCallstackHandler::FMallocCallstackHandler(FMalloc* InMalloc)
	: UsedMalloc(InMalloc)
	, Initialized(false)
{
	DisabledTLS = FPlatformTLS::AllocTlsSlot();
	uint64_t Count = 0;
	FPlatformTLS::SetTlsValue(DisabledTLS, (void*)Count);
}

void FMallocCallstackHandler::Init()
{
	if (Initialized)
	{
		return;
	}
	CallStackInfoArray.Reserve(1250000);	/* Needs to be big enough to never resize! */
	CallStackMapKeyToCallStackIndexMap.Reserve(1250000);
	Initialized = true;
	GMallocCallstackHandler = this;
}

//***********************************
// FMalloc
//***********************************

void* FMallocCallstackHandler::Malloc(SIZE_T Size, uint32 Alignment)
{
	if (IsDisabled())
	{
		return UsedMalloc->Malloc(Size, Alignment);
	}

	FScopeDisableMallocCallstackHandler Disable;

	int32 CallStackIndex = GetCallStackIndex();

	FScopeLock Lock(&CriticalSection);

	void* Ptr = UsedMalloc->Malloc(Size, Alignment);

	SIZE_T AllocatedSize = Size;
	if (UsedMalloc->GetAllocationSize(Ptr, AllocatedSize))
	{
		TrackMalloc(Ptr, (uint32)AllocatedSize, CallStackIndex);
	}
	else
	{
		TrackMalloc(Ptr, (uint32)Size, CallStackIndex);
	}
	return Ptr;
}

void* FMallocCallstackHandler::Realloc(void* OldPtr, SIZE_T NewSize, uint32 Alignment)
{
	if (IsDisabled())
	{
		return UsedMalloc->Realloc(OldPtr, NewSize, Alignment);
	}

	FScopeDisableMallocCallstackHandler Disable;

	int32 CallStackIndex = GetCallStackIndex();
	SIZE_T OldSize = 0;

	FScopeLock Lock(&CriticalSection);

	UsedMalloc->GetAllocationSize(OldPtr, OldSize);

	void* NewPtr = UsedMalloc->Realloc(OldPtr, NewSize, Alignment);

	SIZE_T AllocatedSize = NewSize;
	if (UsedMalloc->GetAllocationSize(NewPtr, AllocatedSize))
	{
		TrackRealloc(OldPtr, NewPtr, (uint32)AllocatedSize, (uint32)OldSize, CallStackIndex);
	}
	else
	{
		TrackRealloc(OldPtr, NewPtr, (uint32)NewSize, (uint32)OldSize, CallStackIndex);
	}
	return NewPtr;
}

void FMallocCallstackHandler::Free(void* Ptr)
{
	if (IsDisabled() || Ptr == nullptr)
	{
		return UsedMalloc->Free(Ptr);
	}

	FScopeDisableMallocCallstackHandler Disable;

	int32 CallStackIndex = GetCallStackIndex();

	FScopeLock Lock(&CriticalSection);
	SIZE_T OldSize = 0;
	UsedMalloc->GetAllocationSize(Ptr, OldSize);
	UsedMalloc->Free(Ptr);
	TrackFree(Ptr, (uint32)OldSize, CallStackIndex);
}

void FMallocCallstackHandler::TrackRealloc(void* OldPtr, void* NewPtr, uint32 NewSize, uint32 OldSize, int32 CallStackIndex)
{
	if (OldPtr == nullptr)
	{
		TrackMalloc(NewPtr, NewSize, CallStackIndex);
	}
	else
	{
		if (OldPtr != NewPtr)
		{
			if (OldPtr)
			{
				TrackFree(OldPtr, OldSize, CallStackIndex);
			}
			if (NewPtr)
			{
				TrackMalloc(NewPtr, NewSize, CallStackIndex);
			}
		}
	}
}


int32 FMallocCallstackHandler::GetCallStackIndex()
{
	// Index of the callstack in CallStackInfoArray.
	int32 Index = INDEX_NONE;

	// Capture callstack and create FCallStackMapKey.
	uint64 FullCallStack[MaxCallStackDepth + CallStackEntriesToSkipCount] = { 0 };
	FPlatformStackWalk::CaptureStackBackTrace(&FullCallStack[0], MaxCallStackDepth + CallStackEntriesToSkipCount);
	// Skip first n entries as they are inside the allocator.
	uint64* CallStack = &FullCallStack[CallStackEntriesToSkipCount];
	FCallStackMapKey CallStackMapKey(CallStack);
	RWLock.ReadLock();
	int32* IndexPtr = CallStackMapKeyToCallStackIndexMap.Find(CallStackMapKey);
	if (IndexPtr)
	{
		// Use index if found
		Index = *IndexPtr;
		RWLock.ReadUnlock();
	}
	else
	{
		// new call stack, add to array and set index.
		RWLock.ReadUnlock();
		FCallStackInfo CallStackInfo;
		CallStackInfo.Count = MaxCallStackDepth;
		for (int32 i = 0; i < MaxCallStackDepth; i++)
		{
			if (!CallStack[i] && CallStackInfo.Count == MaxCallStackDepth)
			{
				CallStackInfo.Count = i;
			}
			CallStackInfo.FramePointers[i] = CallStack[i];
		}

		RWLock.WriteLock();
		Index = CallStackInfoArray.Num();
		CallStackInfoArray.Append(&CallStackInfo, 1);
		CallStackMapKey.CallStack = &CallStackInfoArray[Index].FramePointers[0];
		CallStackMapKeyToCallStackIndexMap.Add(CallStackMapKey, Index);
		RWLock.WriteUnlock();
	}
	return Index;
}

FORCENOINLINE void FMallocCallstackHandler::DumpStackTraceToLog(int32 StackIndex)
{
#if !NO_LOGGING
	// Walk the stack and dump it to the allocated memory.
	const SIZE_T StackTraceStringSize = 16384;
	ANSICHAR StackTraceString[StackTraceStringSize];
	{
		StackTraceString[0] = 0;
		uint32 CurrentDepth = 0;
		while (CurrentDepth < MaxCallStackDepth && CallStackInfoArray[StackIndex].FramePointers[CurrentDepth] != 0)
		{
			FPlatformStackWalk::ProgramCounterToHumanReadableString(CurrentDepth, CallStackInfoArray[StackIndex].FramePointers[CurrentDepth], &StackTraceString[0], StackTraceStringSize, reinterpret_cast<FGenericCrashContext*>(0));
			FCStringAnsi::StrncatTruncateDest(StackTraceString, StackTraceStringSize, LINE_TERMINATOR_ANSI);
			CurrentDepth++;
		}
	}
	// Dump the error and flush the log.
	// ELogVerbosity::Error to make sure it gets printed in log for convenience.
	FDebug::LogFormattedMessageWithCallstack(LogOutputDevice.GetCategoryName(), __FILE__, __LINE__, TEXT("FMallocCallstackHandler::DumpStackTraceToLog"), ANSI_TO_TCHAR(&StackTraceString[0]), ELogVerbosity::Error);
	GLog->Flush();
#endif
}
