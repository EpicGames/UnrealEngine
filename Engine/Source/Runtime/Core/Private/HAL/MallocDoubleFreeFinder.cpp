// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	FMallocDoubleFreeFinder.cpp: Memoory tracking allocator
=============================================================================*/
#include "HAL/MallocDoubleFreeFinder.h"

#include "HAL/MemoryBase.h"
#include "HAL/UnrealMemory.h"
#include "Misc/Parse.h"
#include "Misc/ScopeLock.h"

class FOutputDevice;
class UWorld;

CORE_API FMallocDoubleFreeFinder* GMallocDoubleFreeFinder;
CORE_API bool GMallocDoubleFreeFinderEnabled = false;

FMallocDoubleFreeFinder::FMallocDoubleFreeFinder(FMalloc* InMalloc)
	: FMallocCallstackHandler(InMalloc)
{
}

void FMallocDoubleFreeFinder::Init()
{
	if (Initialized)
	{
		return;
	}

	FMallocCallstackHandler::Init();
	TrackedFreeAllocations.Reserve(6000000);
	TrackedCurrentAllocations.Reserve(8000000);
}


void FMallocDoubleFreeFinder::TrackMalloc(void* Ptr, uint32 Size, int32 CallStackIndex)
{
	if (Ptr != nullptr)
	{
		TrackedAllocationData* AlreadyThere = TrackedCurrentAllocations.Find(Ptr);
		if (AlreadyThere != nullptr)
		{
			static TrackedAllocationData* AlreadyThereStatic = AlreadyThere;
			TrackSpecial(Ptr);
			PLATFORM_BREAK();
		}
		TrackedCurrentAllocations.Add(Ptr, TrackedAllocationData(Size, CallStackIndex));
	}
}

void FMallocDoubleFreeFinder::TrackFree(void* Ptr, uint32 OldSize, int32 CallStackIndex)
{
	TrackedAllocationData Removed;
	if (!TrackedCurrentAllocations.RemoveAndCopyValue(Ptr, Removed))
	{
		static TrackedAllocationData WhatHaveWeHere;
		//memory we don't know about
		TrackedAllocationData*AlreadyThere = TrackedFreeAllocations.Find(Ptr);
		WhatHaveWeHere = *AlreadyThere;
		DumpStackTraceToLog(AlreadyThere->CallStackIndex);
		PLATFORM_BREAK();
	}
	else
	{
		if (OldSize != 0 && OldSize != Removed.Size)
		{
			PLATFORM_BREAK();
		}
		TrackedFreeAllocations.Add(Ptr, TrackedAllocationData(OldSize, CallStackIndex));
	}
}

// This can be set externally, if it is we try and find what freed it before.
void * GTrackFreeSpecialPtr = nullptr;

// Can be called to find out what freed something last
void FMallocDoubleFreeFinder::TrackSpecial(void* Ptr)
{
	FScopeDisableMallocCallstackHandler Disable;
	FScopeLock Lock(&CriticalSection);
	static TrackedAllocationData WhatHaveWeHere;	// Made static so it should be visible in the debugger
	TrackedAllocationData*AlreadyThere;
	TrackedAllocationData Removed;
	if (GTrackFreeSpecialPtr != nullptr)
	{
		if (!TrackedCurrentAllocations.RemoveAndCopyValue(GTrackFreeSpecialPtr, Removed))
		{
			// Untracked memory!!
			AlreadyThere = TrackedFreeAllocations.Find(GTrackFreeSpecialPtr);
			WhatHaveWeHere = *AlreadyThere;
			DumpStackTraceToLog(AlreadyThere->CallStackIndex);
			PLATFORM_BREAK();
		}

	}
	if (!TrackedCurrentAllocations.RemoveAndCopyValue(Ptr, Removed))
	{
		// Untracked memory!!
		AlreadyThere = TrackedFreeAllocations.Find(Ptr);
		WhatHaveWeHere = *AlreadyThere;
		DumpStackTraceToLog(AlreadyThere->CallStackIndex);
		PLATFORM_BREAK();
	}

	// Untracked memory!!
	AlreadyThere = TrackedFreeAllocations.Find(Ptr);
	if (AlreadyThere)
	{
		// found an exact match
		WhatHaveWeHere = *AlreadyThere;
		DumpStackTraceToLog(AlreadyThere->CallStackIndex);
		PLATFORM_BREAK();
	}
	else
	{
		// look for the pointer within another allocation that was previously freed
		auto MyIterator = TrackedFreeAllocations.CreateIterator();
		for (; MyIterator; ++MyIterator)
		{
			intptr_t MyKey = (intptr_t)MyIterator.Key();
			intptr_t MyPtr = (intptr_t)Ptr;
			TrackedAllocationData *AlreadyThere2 = &MyIterator.Value();
			if (MyPtr >= MyKey)
			{
				if (MyPtr < (MyKey + (intptr_t)AlreadyThere2->Size))
				{
					WhatHaveWeHere = *AlreadyThere2;
					DumpStackTraceToLog(AlreadyThere2->CallStackIndex);
					PLATFORM_BREAK();
				}
			}
		}
	}
}

#if UE_ALLOW_EXEC_COMMANDS
bool FMallocDoubleFreeFinder::Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	if (FParse::Command(&Cmd, TEXT("DoubleFreeFinderCrash")))
	{
		void * test;
		test = FMemory::Malloc(128);
		FMemory::Free(test);
		FMemory::Free(test); //-V586
		return true;
	}

	return UsedMalloc->Exec(InWorld, Cmd, Ar);
}
#endif // UE_ALLOW_EXEC_COMMANDS

FMalloc* FMallocDoubleFreeFinder::OverrideIfEnabled(FMalloc*InUsedAlloc)
{
	if (GMallocDoubleFreeFinderEnabled)
	{
		GMallocDoubleFreeFinder = new FMallocDoubleFreeFinder(InUsedAlloc);
		GMallocDoubleFreeFinder->Init();
		return GMallocDoubleFreeFinder;
	}
	return InUsedAlloc;
}
