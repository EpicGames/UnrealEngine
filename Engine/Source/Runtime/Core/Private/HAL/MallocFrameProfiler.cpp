// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	FMallocFrameProfiler.cpp: Memoory tracking allocator
=============================================================================*/
#include "HAL/MallocFrameProfiler.h"

#include "HAL/MemoryBase.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Misc/Parse.h"
#include "Misc/ScopeLock.h"

class FOutputDevice;
class UWorld;

DECLARE_LOG_CATEGORY_EXTERN(LogMallocFrameProfiler, Log, All);
DEFINE_LOG_CATEGORY(LogMallocFrameProfiler);

CORE_API FMallocFrameProfiler* GMallocFrameProfiler;
CORE_API bool GMallocFrameProfilerEnabled = false;

FMallocFrameProfiler::FMallocFrameProfiler(FMalloc* InMalloc)
	: FMallocCallstackHandler(InMalloc)
	, bEnabled(false)
	, FrameCount(0)
	, EntriesToOutput(15)
{
}

void FMallocFrameProfiler::Init()
{
	if (Initialized)
	{
		return;
	}
	FMallocCallstackHandler::Init();

	TrackedCurrentAllocations.Reserve(8000000);
	CallStackStatsArray.Reserve(8000000);
}

void FMallocFrameProfiler::TrackMalloc(void* Ptr, uint32 Size, int32 CallStackIndex)
{
	if (Ptr != nullptr)
	{
		if (CallStackStatsArray.Num() <= CallStackIndex)
		{
			CallStackStatsArray.AddDefaulted(CallStackIndex - CallStackStatsArray.Num() + 1);
		}

		CallStackStatsArray[CallStackIndex].CallStackIndex = CallStackIndex;
		CallStackStatsArray[CallStackIndex].Mallocs++;

		if (CallStackStatsArray[CallStackIndex].UniqueFrames == 0)
		{
			CallStackStatsArray[CallStackIndex].UniqueFrames++;
		}

		if (CallStackStatsArray[CallStackIndex].LastFrameSeen != FrameCount)
		{
			CallStackStatsArray[CallStackIndex].UniqueFrames++;
			CallStackStatsArray[CallStackIndex].LastFrameSeen = FrameCount;
		}

		TrackedCurrentAllocations.Add(Ptr, CallStackIndex);
	}
}

void FMallocFrameProfiler::TrackFree(void* Ptr, uint32 OldSize, int32 CallStackIndex)
{
	int32* CallStackIndexMalloc = TrackedCurrentAllocations.Find(Ptr);
	if (CallStackIndexMalloc!=nullptr)
	{
		if (CallStackStatsArray.Num() <= *CallStackIndexMalloc)
		{
			// it cant be
			PLATFORM_BREAK();
		}

		CallStackStatsArray[*CallStackIndexMalloc].UsageCount++;
		CallStackStatsArray[*CallStackIndexMalloc].Frees++;
	}
}

void FMallocFrameProfiler::TrackRealloc(void* OldPtr, void* NewPtr, uint32 NewSize, uint32 OldSize, int32 CallStackIndex)
{
	if (OldPtr == nullptr)
	{
		TrackMalloc(NewPtr, NewSize, CallStackIndex);
	}
	else
	{
		if (OldPtr != NewPtr)
		{
			if (OldPtr)
			{
				TrackFree(OldPtr, OldSize, CallStackIndex);
			}
			if (NewPtr)
			{
				TrackMalloc(NewPtr, NewSize, CallStackIndex);
			}
		}
	}
}

bool FMallocFrameProfiler::IsDisabled()
{
	return FMallocCallstackHandler::IsDisabled() || !bEnabled;
}

void FMallocFrameProfiler::UpdateStats()
{
	UsedMalloc->UpdateStats();

	if (!bEnabled)
	{
		return;
	}

	FScopeLock Lock(&CriticalSection);
	TrackedCurrentAllocations.Reset();

	if (FrameCount > 0)
	{
		FrameCount--;
		return;
	}
	
	bEnabled = false;

	CallStackStatsArray.Sort
	(
		[](const FCallStackStats& A, const FCallStackStats& B)
		{
			return A.Mallocs > B.Mallocs;
		}
	);

	for (int32 CallStackIndex=0; CallStackIndex< CallStackStatsArray.Num(); CallStackIndex++)
	{
		if (CallStackStatsArray[CallStackIndex].CallStackIndex != 0)
		{
			UE_LOGF(LogMallocFrameProfiler, Display, "---- Call Stack Stats for Index %d Mallocs %d Frees %d Pairs %d FramesSeen %d Avg %.2f ----",
				CallStackStatsArray[CallStackIndex].CallStackIndex,
				CallStackStatsArray[CallStackIndex].Mallocs,
				CallStackStatsArray[CallStackIndex].Frees,
				CallStackStatsArray[CallStackIndex].UsageCount,
				CallStackStatsArray[CallStackIndex].UniqueFrames,
				((float)CallStackStatsArray[CallStackIndex].Mallocs / (float)CallStackStatsArray[CallStackIndex].UniqueFrames));
			DumpStackTraceToLog(CallStackStatsArray[CallStackIndex].CallStackIndex);
		}
		
		if (CallStackIndex == EntriesToOutput)
		{
			break;
		}
	}

	CallStackInfoArray.Reset();
	CallStackMapKeyToCallStackIndexMap.Reset();
	CallStackStatsArray.Reset();
}

#if UE_ALLOW_EXEC_COMMANDS
bool FMallocFrameProfiler::Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	if (FParse::Command(&Cmd, TEXT("MallocFrameProfiler")))
	{
		if (!FParse::Value(Cmd, TEXT("FrameCount="), FrameCount))
		{
			FrameCount = 0;
		}

		if (!FParse::Value(Cmd, TEXT("Entries="), EntriesToOutput))
		{
			EntriesToOutput = 15;
		}

		bEnabled = true;
		return true;
	}

	return UsedMalloc->Exec(InWorld, Cmd, Ar);
}
#endif // UE_ALLOW_EXEC_COMMANDS

FMalloc* FMallocFrameProfiler::OverrideIfEnabled(FMalloc*InUsedAlloc)
{
	if (GMallocFrameProfilerEnabled)
	{
		GMallocFrameProfiler = new FMallocFrameProfiler(InUsedAlloc);
		GMallocFrameProfiler->Init();
		return GMallocFrameProfiler;
	}
	return InUsedAlloc;
}
