// Copyright Epic Games, Inc. All Rights Reserved.

#include "HAL/MallocJemalloc.h"

// Only use for supported platforms
#if PLATFORM_SUPPORTS_JEMALLOC

#define JEMALLOC_NO_DEMANGLE	// use stable function names prefixed with je_
#include "jemalloc.h"

#ifndef JEMALLOC_VERSION
#error JEMALLOC_VERSION not defined!
#endif

void* FMallocJemalloc::Malloc( SIZE_T Size, uint32 Alignment )
{
	MEM_TIME(MemTime -= FPlatformTime::Seconds());

	void* Free = NULL;
	if( Alignment != DEFAULT_ALIGNMENT )
	{
		Alignment = FMath::Max(Size >= 16 ? (uint32)16 : (uint32)8, Alignment);

		// use aligned_alloc when allocating exact multiplies of an alignment
		// use the fact that Alignment is power of 2 and avoid %, but check it
		checkSlow(0 == (Alignment & (Alignment - 1)));
		if ((Size & (Alignment - 1)) == 0)
		{
			Free = je_aligned_alloc(Alignment, Size);
		}
		else if (0 != je_posix_memalign(&Free, Alignment, Size))
		{
			Free = NULL;	// explicitly nullify
		};
	}
	else
	{
		Free = je_malloc( Size );
	}

	if( !Free )
	{
		OutOfMemory();
	}

	MEM_TIME(MemTime += FPlatformTime::Seconds());
	return Free;	
}

void* FMallocJemalloc::Realloc( void* Ptr, SIZE_T NewSize, uint32 Alignment )
{
	MEM_TIME(MemTime -= FPlatformTime::Seconds());

	SIZE_T OldSize = 0;
	if (Ptr)
	{
		OldSize = je_malloc_usable_size(Ptr);
	}
	void* NewPtr = NULL;
	if (Alignment != DEFAULT_ALIGNMENT)
	{
		NewPtr = Malloc(NewSize, Alignment);
		if (Ptr)
		{
			FMemory::Memcpy(NewPtr, Ptr, OldSize);
			Free(Ptr);
		}
	}
	else
	{
		NewPtr = je_realloc(Ptr, NewSize);
	}

	MEM_TIME(MemTime += FPlatformTime::Seconds());
	return NewPtr;
}
	
void FMallocJemalloc::Free( void* Ptr )
{
	if( !Ptr )
	{
		return;
	}

	MEM_TIME(MemTime -= FPlatformTime::Seconds());

	je_free(Ptr);

	MEM_TIME(MemTime += FPlatformTime::Seconds());
}

namespace
{
	void JemallocStatsPrintCallback(void *UserData, const char *String)
	{
		FOutputDevice* Ar = reinterpret_cast< FOutputDevice* >(UserData);

		check(Ar);
		if (Ar)
		{
			FString Sanitized(ANSI_TO_TCHAR(String));
			Sanitized.ReplaceInline(TEXT("\n"), TEXT(""), ESearchCase::CaseSensitive);
			Sanitized.ReplaceInline(TEXT("\r"), TEXT(""), ESearchCase::CaseSensitive);
			Ar->Logf(TEXT("%s"), *Sanitized);
		}
	}
}

void FMallocJemalloc::DumpAllocatorStats( FOutputDevice& Ar ) 
{
	MEM_TIME(Ar.Logf( TEXT("Seconds     % 5.3f"), MemTime ));
	
	// "g" omits static stats, "a" omits per-arena stats, "l" omits large object stats
	// see http://www.canonware.com/download/jemalloc/jemalloc-latest/doc/jemalloc.html for detailed opts explanation.
	je_malloc_stats_print(JemallocStatsPrintCallback, &Ar, "gla");
}

bool FMallocJemalloc::GetAllocationSize(void *Original, SIZE_T &SizeOut)
{
	SizeOut = je_malloc_usable_size(Original);
	return true;
}

#undef DEBUG_FILL_FREED
#undef DEBUG_FILL_NEW

#endif // PLATFORM_SUPPORTS_JEMALLOC
