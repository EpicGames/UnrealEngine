// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MallocLeakDetection.cpp: Helper class to track memory allocations
=============================================================================*/

#include "HAL/MallocLeakDetection.h"
#include "HAL/MallocLeakDetectionProxy.h"
#include "Logging/LogMacros.h"
#include "HAL/PlatformTLS.h"
#include "HAL/IConsoleManager.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformStackWalk.h"
#include "Misc/Parse.h"
#include "Misc/Paths.h"
#include "Logging/LogMacros.h"
#include "Misc/OutputDeviceArchiveWrapper.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Misc/DateTime.h"
#include "ProfilingDebugging/ProfilingHelpers.h"

#if MALLOC_LEAKDETECTION

/**
 *	Need forced-initialization of this data as it can be used during global
*	ctors.
 */
struct FMallocLeakDetectionStatics
{
	uint32 ContextsTLSID = FPlatformTLS::InvalidTlsSlot;
	uint32 SuppressDetectionCountTLSID = FPlatformTLS::InvalidTlsSlot;

	FMallocLeakDetectionStatics()
	{
		SuppressDetectionCountTLSID = FPlatformTLS::AllocTlsSlot();
		ContextsTLSID = FPlatformTLS::AllocTlsSlot();
	}

	static FMallocLeakDetectionStatics& Get()
	{
		static FMallocLeakDetectionStatics Singleton;
		return Singleton;
	}
};

FMallocLeakDetection::FMallocLeakDetection()
	: bRecursive(false)
	, bCaptureAllocs(false)
	, MinAllocationSize(0)
	, TotalTracked(0)
	, AllocsWithoutCompact(0)
{
	Contexts.Empty(16);
}

FMallocLeakDetection& FMallocLeakDetection::Get()
{
	static FMallocLeakDetection Singleton;
	return Singleton;
}

FMallocLeakDetection::~FMallocLeakDetection()
{
	bRecursive = true; // OpenPointers.Empty() may reallocate / free.
	OpenPointers.Empty(); // clean up the state
	bRecursive = false;
	SetAllocationCollection(false); // disable collection to avoid a call back to this instance when its members are destroyed
}


void FMallocLeakDetection::SetDisabledForThisThread(const bool Disabled)
{
	int Count = (int)(size_t)FPlatformTLS::GetTlsValue(FMallocLeakDetectionStatics::Get().SuppressDetectionCountTLSID);
	Count += Disabled ? 1 : -1;
	check(Count >= 0);

	FPlatformTLS::SetTlsValue(FMallocLeakDetectionStatics::Get().SuppressDetectionCountTLSID, (void*)(size_t)Count);
}

bool FMallocLeakDetection::IsDisabledForThisThread() const
{
	return !!FPlatformTLS::GetTlsValue(FMallocLeakDetectionStatics::Get().SuppressDetectionCountTLSID);
}

void FMallocLeakDetection::PushContext(const TCHAR* Context)
{
	FScopeLock Lock(&AllocatedPointersCritical);

	TArray<FContextString>* TLContexts = (TArray<FContextString>*)FPlatformTLS::GetTlsValue(FMallocLeakDetectionStatics::Get().ContextsTLSID);
	if (!TLContexts)
	{
		TLContexts = new TArray<FContextString>();
		FPlatformTLS::SetTlsValue(FMallocLeakDetectionStatics::Get().ContextsTLSID, TLContexts);
	}

	bRecursive = true;

	FContextString Str;
	FCString::Strncpy(Str.Buffer, Context, UE_ARRAY_COUNT(Str.Buffer));
	TLContexts->Push(Str);
	bRecursive = false;
}

void FMallocLeakDetection::PopContext()
{
	TArray<FContextString>* TLContexts = (TArray<FContextString>*)FPlatformTLS::GetTlsValue(FMallocLeakDetectionStatics::Get().ContextsTLSID);
	check(TLContexts);
	TLContexts->Pop(EAllowShrinking::No);
}

void FMallocLeakDetection::AddCallstack(FCallstackTrack& Callstack)
{
	FScopeLock Lock(&AllocatedPointersCritical);
	CallstackHashType CallstackHash = Callstack.GetHash();
	FCallstackTrack& UniqueCallstack = UniqueCallstacks.FindOrAdd(CallstackHash);
	//if we had a hash collision bail and lose the data rather than corrupting existing data.
	if ((UniqueCallstack.Count > 0 || UniqueCallstack.NumCheckPoints > 0) && UniqueCallstack != Callstack)
	{
		ensureMsgf(false, TEXT("Callstack hash collision.  Throwing away new stack."));
		return;
	}

	if (UniqueCallstack.Count == 0 && UniqueCallstack.NumCheckPoints == 0)
	{
		UniqueCallstack = Callstack;
	}
	else
	{
		UniqueCallstack.Size += Callstack.Size;
		UniqueCallstack.LastFrame = Callstack.LastFrame;
	}
	UniqueCallstack.Count++;
	TotalTracked += Callstack.Size;
}

void FMallocLeakDetection::RemoveCallstack(FCallstackTrack& Callstack)
{
	FScopeLock Lock(&AllocatedPointersCritical);
	CallstackHashType CallstackHash = Callstack.GetHash();
	FCallstackTrack* UniqueCallstack = UniqueCallstacks.Find(CallstackHash);
	if (UniqueCallstack)
	{
		UniqueCallstack->Count--;
		UniqueCallstack->Size -= Callstack.Size;

		if (UniqueCallstack->Count == 0)
		{
			UniqueCallstack = nullptr;
			UniqueCallstacks.Remove(CallstackHash);
		}

		TotalTracked -= Callstack.Size;
	}
}

void FMallocLeakDetection::SetAllocationCollection(bool bEnabled, int32 Size)
{
	FScopeLock Lock(&AllocatedPointersCritical);
	bCaptureAllocs = bEnabled;

	if (bEnabled)
	{
		MinAllocationSize = Size;
	}
}

void FMallocLeakDetection::GetOpenCallstacks(TArray<CallstackHashType>& OutCallstacks, SIZE_T& OutTotalSize, const FMallocLeakReportOptions& Options /*= FMallocLeakReportOptions()*/)
{
	// Make sure we preallocate memory to avoid
	OutCallstacks.Empty(UniqueCallstacks.Num() + 32);

	OutTotalSize = 0;

	TMap<CallstackHashType, float> HashesToAllocRate;
	HashesToAllocRate.Empty(UniqueCallstacks.Num() + 32);

	{
		FScopeLock Lock(&AllocatedPointersCritical);

		// UniqueCallstacks is modified on HashesToAllocRate.Add which will assert
		bRecursive = true;

		const int kRequiredRateCheckpoints = 3;

		for (const auto& Pair : UniqueCallstacks)
		{
			const FCallstackTrack& Callstack = Pair.Value;

			// filter based on rate (discard if we have less, or not enough data)
			if (Options.RateFilter > 0.0f && (Callstack.NumCheckPoints < kRequiredRateCheckpoints || Callstack.BytesPerFrame < Options.RateFilter))
			{
				continue;
			}

			// Filter out allocations that free/resize down?
			if (Options.OnlyNonDeleters
				&& (KnownDeleters.Contains(Callstack.CachedHash) || KnownTrimmers.Contains(Callstack.CachedHash)))
			{
				continue;
			}

			// frame number?
			if (Options.FrameStart > Callstack.LastFrame)
			{
				continue;
			}

			if (Options.FrameEnd && Options.FrameEnd < Callstack.LastFrame)
			{
				continue;
			}

			// Filter on size?
			if (Callstack.Size < Options.SizeFilter)
			{
				continue;
			}

			HashesToAllocRate.Add(Callstack.CachedHash, Callstack.BytesPerFrame);

			OutCallstacks.Add(Pair.Key);

			OutTotalSize += Callstack.Size;
		}

		// now sort
		OutCallstacks.Sort([this, &Options, &HashesToAllocRate](CallstackHashType lhs, CallstackHashType rhs) {

			if (Options.SortBy == FMallocLeakReportOptions::ESortOption::SortRate)
			{
				return HashesToAllocRate[lhs] >= HashesToAllocRate[rhs];
			}

			FCallstackTrack& Left = UniqueCallstacks[lhs];
			FCallstackTrack& Right = UniqueCallstacks[rhs];

			if (Options.SortBy == FMallocLeakReportOptions::ESortOption::SortHash)
			{
				return Left.CachedHash < Right.CachedHash;
			}

			// else sort by Ascending size
			return Left.Size >= Right.Size;
		});

		bRecursive = false;
	}
}

void FMallocLeakDetection::CheckpointLinearFit()
{
	FScopeLock Lock(&AllocatedPointersCritical);

	float FrameNum = float(GFrameCounter);
	float FrameNum2 = FrameNum * FrameNum;

	for (auto& Pair : UniqueCallstacks)
	{
		FCallstackTrack& Callstack = Pair.Value;

		Callstack.NumCheckPoints++;
		Callstack.SumOfFramesNumbers += FrameNum;
		Callstack.SumOfFramesNumbersSquared += FrameNum2;
		Callstack.SumOfMemory += float(Callstack.Size);
		Callstack.SumOfMemoryTimesFrameNumber += float(Callstack.Size) * FrameNum;
		Callstack.GetLinearFit();
	}
}

void FMallocLeakDetection::FCallstackTrack::GetLinearFit()
{
	Baseline = 0.0f;
	BytesPerFrame = 0.0f;
	float DetInv = (float(NumCheckPoints) * SumOfFramesNumbersSquared) - (2.0f * SumOfFramesNumbers);
	if (!NumCheckPoints || DetInv == 0.0f)
	{
		return;
	}
	float Det = 1.0f / DetInv;

	float AtAInv[2][2];

	AtAInv[0][0] = Det * SumOfFramesNumbersSquared;
	AtAInv[0][1] = -Det * SumOfFramesNumbers;
	AtAInv[1][0] = -Det * SumOfFramesNumbers;
	AtAInv[1][1] = Det * float(NumCheckPoints);

	Baseline = AtAInv[0][0] * SumOfMemory + AtAInv[0][1] * SumOfMemoryTimesFrameNumber;
	BytesPerFrame = AtAInv[1][0] * SumOfMemory + AtAInv[1][1] * SumOfMemoryTimesFrameNumber;
}

int32 FMallocLeakDetection::DumpOpenCallstacks(const TCHAR* FileName, const FMallocLeakReportOptions& Options /*= FMallocLeakReportOptions()*/)
{
	check(FCString::Strlen(FileName));

	SIZE_T ReportedSize = 0;
	SIZE_T TotalSize = 0;
	TArray<CallstackHashType> SortedKeys;

	GetOpenCallstacks(SortedKeys, ReportedSize, Options);

	// Nothing to do
	if (SortedKeys.Num() == 0)
	{
		return 0;
	}

	FOutputDevice* ReportAr = nullptr;
	FArchive* FileAr = nullptr;
	FOutputDeviceArchiveWrapper* FileArWrapper = nullptr;

	if (Options.OutputDevice)
	{
		ReportAr = Options.OutputDevice;
	}
	else
	{
#if ALLOW_DEBUG_FILES
		const FString PathName = *(FPaths::ProfilingDir() + TEXT("memreports/"));
		IFileManager::Get().MakeDirectory(*PathName);

		FString FilePath = PathName + CreateProfileFilename(FileName, TEXT(""), true);

		FileAr = IFileManager::Get().CreateDebugFileWriter(*FilePath);
		FileArWrapper = new FOutputDeviceArchiveWrapper(FileAr);
		ReportAr = FileArWrapper;
#else
		// We can't create debug files because we aren't allowed, so bail!
		return 0;
#endif
	}

	const float InvToMb = 1.0 / (1024 * 1024);
	FPlatformMemoryStats MemoryStats = FPlatformMemory::GetStats();

	ReportAr->Logf(TEXT("Current Time: %s, Current Frame %d"), *FDateTime::Now().ToString(TEXT("%m.%d-%H.%M.%S")), GFrameCounter);

	ReportAr->Logf(TEXT("Current Memory: %.02fMB (Peak: %.02fMB)."),
		(float)MemoryStats.UsedPhysical * InvToMb,
		(float)MemoryStats.PeakUsedPhysical * InvToMb);

	ReportAr->Logf(TEXT("Tracking %d callstacks that hold %.02fMB"), UniqueCallstacks.Num(), (float)TotalTracked * InvToMb);

	ReportAr->Logf(TEXT("Allocation filter: %dKB"), MinAllocationSize / 1024);
	ReportAr->Logf(TEXT("Report filter: %dKB"), Options.SizeFilter / 1024);
	ReportAr->Logf(TEXT("Have %i open callstacks at frame: %i"), UniqueCallstacks.Num(),(int32)GFrameCounter);


	ReportAr->Logf(TEXT("Dumping %d callstacks that hold more than %uKBs and total %uKBs"),
		SortedKeys.Num(), Options.SizeFilter / 1024, ReportedSize / 1024);

	const int32 MaxCallstackLineChars = 2048;
	char CallstackString[MaxCallstackLineChars];
	TCHAR CallstackStringWide[MaxCallstackLineChars];
	FMemory::Memzero(CallstackString);

	for (const auto& Key : SortedKeys)
	{
		FCallstackTrack Callstack;

		{
			FScopeLock Lock(&AllocatedPointersCritical);
			if (UniqueCallstacks.Contains(Key) == false)
				continue;

			Callstack = UniqueCallstacks[Key];
		}

		bool KnownDeleter = KnownDeleters.Contains(Callstack.GetHash());
		bool KnownTrimmer = KnownTrimmers.Contains(Callstack.CachedHash);

		ReportAr->Logf(TEXT("\nAllocSize: %i KB, Num: %i, FirstFrame %i, LastFrame %i, KnownDeleter: %d, KnownTrimmer: %d, Alloc Rate %.2fB/frame")
			, Callstack.Size / 1024
			, Callstack.Count
			, Callstack.FirstFrame
			, Callstack.LastFrame
			, KnownDeleter
			, KnownTrimmer
			, Callstack.BytesPerFrame);

		for (int32 i = 0; i < FCallstackTrack::Depth; ++i)
		{
			FPlatformStackWalk::ProgramCounterToHumanReadableString(i, Callstack.Callstack[i], CallstackString, MaxCallstackLineChars);

			if (Callstack.Callstack[i] && FCStringAnsi::Strlen(CallstackString) > 0)
			{
				//convert ansi -> tchar without mallocs in case we are in OOM handler.
				for (int32 CurrChar = 0; CurrChar < MaxCallstackLineChars; ++CurrChar)
				{
					CallstackStringWide[CurrChar] = CallstackString[CurrChar];
				}

				ReportAr->Logf(TEXT("%s"), CallstackStringWide);
			}
			FMemory::Memzero(CallstackString);
		}

		TArray<FString> SortedContexts;

		FScopeLock Lock(&AllocatedPointersCritical);
		bRecursive = true;

		for (const TPair<void*, FCallstackTrack>& Pair : CopyTemp(OpenPointers))
		{
			if (Pair.Value.CachedHash == Key)
			{
				if (PointerContexts.Contains(Pair.Key))
				{
					SortedContexts.Add(*PointerContexts.Find(Pair.Key));
				}
			}
		}
		bRecursive = false;

		if (SortedContexts.Num())
		{
			ReportAr->Logf(TEXT("%d contexts:"), SortedContexts.Num(), CallstackStringWide);

			SortedContexts.Sort();

			for (const auto& Str : SortedContexts)
			{
				ReportAr->Logf(TEXT("\t%s"), *Str);
			}
		}

		ReportAr->Logf(TEXT("\n"));
	}

	if (FileArWrapper != nullptr)
	{
		FileArWrapper->TearDown();
		delete FileArWrapper;
		delete FileAr;
	}

	return SortedKeys.Num();
}

void FMallocLeakDetection::ClearData()
{
	bool OldCapture = bCaptureAllocs;
	{
		FScopeLock Lock(&AllocatedPointersCritical);
		bCaptureAllocs = false;
	}

	OpenPointers.Empty();
	UniqueCallstacks.Empty();
	PointerContexts.Empty();

	{
		FScopeLock Lock(&AllocatedPointersCritical);
		bCaptureAllocs = OldCapture;
	}
}

void FMallocLeakDetection::Malloc(void* Ptr, SIZE_T Size)
{
	if (Ptr)
	{
		bool Enabled = bCaptureAllocs && IsDisabledForThisThread() == false;

		if (Enabled && (MinAllocationSize == 0 || Size >= MinAllocationSize))
		{
			FScopeLock Lock(&AllocatedPointersCritical);

			if (!bRecursive)
			{
				bRecursive = true;
				FCallstackTrack Callstack;
				FPlatformStackWalk::CaptureStackBackTrace(Callstack.Callstack, FCallstackTrack::Depth);
				Callstack.FirstFrame = (uint32)GFrameCounter;
				Callstack.LastFrame = (uint32)GFrameCounter;
				Callstack.Size = Size;
				AddCallstack(Callstack);
				OpenPointers.Add(Ptr, Callstack);
				AllocsWithoutCompact++;

				TArray<FContextString>* TLContexts = (TArray<FContextString>*)FPlatformTLS::GetTlsValue(FMallocLeakDetectionStatics::Get().ContextsTLSID);

				if (TLContexts && TLContexts->Num())
				{
					FString Context;

					for (int i = TLContexts->Num() - 1; i >= 0; i--)
					{
						if (Context.Len())
						{
							Context += TEXT(".");
						}

						Context += (*TLContexts)[i].Buffer;
					}

					PointerContexts.Add(Ptr, Context);
				}

				if (AllocsWithoutCompact >= 100000)
				{
					OpenPointers.Compact();
					OpenPointers.Shrink();
					AllocsWithoutCompact = 0;
				}

				bRecursive = false;
			}
		}
	}


}

void FMallocLeakDetection::Realloc(void* OldPtr, SIZE_T OldSize, void* NewPtr, SIZE_T NewSize)
{
	if (bCaptureAllocs || OpenPointers.Num())
	{
		FScopeLock Lock(&AllocatedPointersCritical);

		if (bRecursive)
		{
			return;
		}

		// realloc may return the same pointer, if so skip this

		if (OldPtr != NewPtr)
		{
			FCallstackTrack* Callstack = OpenPointers.Find(OldPtr);
			CallstackHashType OldHash = 0;
			bool WasKnownDeleter = false;

			if (Callstack)
			{
				OldSize = Callstack->Size;
				OldHash = Callstack->GetHash();
				WasKnownDeleter = KnownDeleters.Contains(OldHash);
			}

			// Note - malloc and then free so linearfit checkpoints in the callstack are
			// preserved
			Malloc(NewPtr, NewSize);

			// See if we should/need to copy the context across
			if (OldPtr && NewPtr)
			{
				FString* OldContext = PointerContexts.Find(OldPtr);
				FString* NewContext = PointerContexts.Find(NewPtr);

				if (OldContext && !NewContext)
				{
					FString Tmp = *OldContext;
					PointerContexts.Add(NewPtr) = Tmp;
				}
			}

			Free(OldPtr);

			// if we had an old pointer need to do some book keeping...
			if (OldPtr != nullptr)
			{
				// If size is decreasing credit the locations involved with "Trimmer" status.
				if (NewSize < OldSize)
				{
					// some of the below may cause allocations
					bRecursive = true;

					// We may not have a hash if we're reallocing() something that was allocated before
					// we started tracking
					if (OldHash)
					{
						KnownTrimmers.Add(OldHash);
					}

					FCallstackTrack* NewCallstack = OpenPointers.Find(NewPtr);

					// Give the new location immediate credit for being a trimmer. This is for the case where a buffer prior to tracking is
					// sized down. If that happens we don't want the reallocer to look like just an allocation.
					if (NewCallstack)
					{
						if (NewSize > 0)
						{
							KnownTrimmers.Add(NewCallstack->CachedHash);
						}
					}

					bRecursive = false;
				}

				// if we had a tracked callstack and a NewSize > 0 then undo the deleter credit that was applied by the Free() above
				// since it was not actually deallocating memory.
				if (OldHash && NewSize > 0 && WasKnownDeleter == false)
				{
					KnownDeleters.Remove(OldHash);
				}
			}
		}
		else
		{
			// realloc returned the same pointer, if there was a context when the call happened then
			// update it.
			TArray<FContextString>* TLContexts = (TArray<FContextString>*)FPlatformTLS::GetTlsValue(FMallocLeakDetectionStatics::Get().ContextsTLSID);

			if (TLContexts && TLContexts->Num())
			{
				bRecursive = true;
				PointerContexts.Remove(OldPtr);
				PointerContexts.FindOrAdd(NewPtr) = TLContexts->Top().Buffer;
				bRecursive = false;
			}
		}
	}
}

void FMallocLeakDetection::Free(void* Ptr)
{
	if (Ptr)
	{
		if (bCaptureAllocs || OpenPointers.Num())
		{
			FScopeLock Lock(&AllocatedPointersCritical);
			if (!bRecursive)
			{
				bRecursive = true;
				FCallstackTrack* Callstack = OpenPointers.Find(Ptr);
				if (Callstack)
				{
					RemoveCallstack(*Callstack);
					KnownDeleters.Add(Callstack->GetHash());
				}
				OpenPointers.Remove(Ptr);
				bRecursive = false;
			}

			if (PointerContexts.Contains(Ptr))
			{
				PointerContexts.Remove(Ptr);
			}
		}
	}
}

static FMallocLeakDetectionProxy* ProxySingleton = nullptr;

FMallocLeakDetectionProxy::FMallocLeakDetectionProxy(FMalloc* InMalloc)
	: UsedMalloc(InMalloc)
	, Verify(FMallocLeakDetection::Get())
{
	check(ProxySingleton == nullptr);
	ProxySingleton = this;
}

FMallocLeakDetectionProxy& FMallocLeakDetectionProxy::Get()
{
	check(ProxySingleton);
	return *ProxySingleton;
}

#endif // MALLOC_LEAKDETECTION
