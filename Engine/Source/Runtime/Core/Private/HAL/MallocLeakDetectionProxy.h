// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MallocLeakDetectionProxy.h: Helper class to track memory allocations
=============================================================================*/

#pragma once

#include "CoreTypes.h"
#include "HAL/MemoryBase.h"
#include "HAL/UnrealMemory.h"
#include "Containers/Array.h"
#include "Misc/Crc.h"
#include "Containers/UnrealString.h"
#include "Containers/Set.h"
#include "Containers/Map.h"
#include "Misc/ScopeLock.h"
#include "HAL/ThreadSafeBool.h"
#include "HAL/MallocLeakDetection.h"

#if MALLOC_LEAKDETECTION

/**
 * A verifying proxy malloc that takes a malloc to be used and tracks unique callstacks with outstanding allocations
 * to help identify leaks.
 */
class FMallocLeakDetectionProxy : public FMalloc
{
private:
	/** Malloc we're based on, aka using under the hood */
	FMalloc* UsedMalloc;

	/* Verifier object */
	FMallocLeakDetection& Verify;

public:
	explicit FMallocLeakDetectionProxy(FMalloc* InMalloc);

	static FMallocLeakDetectionProxy& Get();

	virtual void* Malloc(SIZE_T Size, uint32 Alignment) override
	{
		void* Result = UsedMalloc->Malloc(Size, Alignment);
		Verify.Malloc(Result, Size);
		return Result;
	}

	virtual void* Realloc(void* OldPtr, SIZE_T NewSize, uint32 Alignment) override
	{
		SIZE_T OldSize(0);
		GetAllocationSize(OldPtr, OldSize);
		void* NewPtr = UsedMalloc->Realloc(OldPtr, NewSize, Alignment);
		Verify.Realloc(OldPtr, OldSize, NewPtr, NewSize);
		return NewPtr;
	}

	virtual void Free(void* Ptr) override
	{
		if (Ptr)
		{
			Verify.Free(Ptr);
			UsedMalloc->Free(Ptr);
		}
	}

	virtual void InitializeStatsMetadata() override
	{
		UsedMalloc->InitializeStatsMetadata();
	}

	virtual void GetAllocatorStats(FGenericMemoryStats& OutStats) override
	{
		UsedMalloc->GetAllocatorStats(OutStats);
	}

	virtual void DumpAllocatorStats(FOutputDevice& Ar) override
	{
		//Verify.DumpOpenCallstacks(1024 * 1024);
		UsedMalloc->DumpAllocatorStats(Ar);
	}

	virtual bool IsInternallyThreadSafe() const override
	{
		return UsedMalloc->IsInternallyThreadSafe();
	}

	virtual bool ValidateHeap() override
	{
		return UsedMalloc->ValidateHeap();
	}

	virtual bool Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override
	{
		return UsedMalloc->Exec(InWorld, Cmd, Ar);
	}

	virtual bool GetAllocationSize(void* Original, SIZE_T& OutSize) override
	{
		return UsedMalloc->GetAllocationSize(Original, OutSize);
	}

	virtual SIZE_T QuantizeSize(SIZE_T Count, uint32 Alignment) override
	{
		return UsedMalloc->QuantizeSize(Count, Alignment);
	}

	virtual void Trim(bool bTrimThreadCaches) override
	{
		return UsedMalloc->Trim(bTrimThreadCaches);
	}
	virtual void SetupTLSCachesOnCurrentThread() override
	{
		return UsedMalloc->SetupTLSCachesOnCurrentThread();
	}

	virtual void MarkTLSCachesAsUsedOnCurrentThread() override
	{
		UsedMalloc->MarkTLSCachesAsUsedOnCurrentThread();
	}

	virtual void MarkTLSCachesAsUnusedOnCurrentThread() override
	{
		UsedMalloc->MarkTLSCachesAsUnusedOnCurrentThread();
	}

	virtual void ClearAndDisableTLSCachesOnCurrentThread() override
	{
		return UsedMalloc->ClearAndDisableTLSCachesOnCurrentThread();
	}

	virtual const TCHAR* GetDescriptiveName() override
	{
		return UsedMalloc->GetDescriptiveName();
	}

	virtual void OnMallocInitialized() override
	{
		UsedMalloc->OnMallocInitialized();
	}
};

#endif // MALLOC_LEAKDETECTIONMALLOC_LEAKDETECTION
