// Copyright Epic Games, Inc. All Rights Reserved.

#include "HAL/MallocLibpas.h"

#if LIBPASMALLOC_ENABLED

#include "HAL/PlatformProcess.h"
#include "HAL/UnrealMemory.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/Fork.h"
#include "bmalloc_heap_ue.h"
#include "pas_scavenger_ue.h"

FMallocLibpas::FMallocLibpas()
{
	if (!FPlatformProcess::SupportsMultithreading())
	{
		pas_scavenger_suspend();
	}
}

void* FMallocLibpas::TryMalloc( SIZE_T Size, uint32 Alignment )
{
	return bmalloc_try_allocate_with_alignment(Size, Alignment);
}

void* FMallocLibpas::Malloc(SIZE_T Size, uint32 Alignment)
{
	return bmalloc_allocate_with_alignment(Size, Alignment);
}

void* FMallocLibpas::TryRealloc(void* Ptr, SIZE_T NewSize, uint32 Alignment)
{
	return bmalloc_try_reallocate_with_alignment(Ptr, NewSize, Alignment, pas_reallocate_free_if_successful);
}

void* FMallocLibpas::Realloc(void* Ptr, SIZE_T NewSize, uint32 Alignment)
{
	return bmalloc_reallocate_with_alignment(Ptr, NewSize, Alignment, pas_reallocate_free_if_successful);
}

void FMallocLibpas::Free(void* Ptr)
{
	bmalloc_deallocate(Ptr);
}

bool FMallocLibpas::GetAllocationSize(void *Original, SIZE_T &SizeOut)
{
	SizeOut = bmalloc_get_allocation_size(Original);
	return true;
}

void FMallocLibpas::Trim(bool bTrimThreadCaches)
{
	if (bTrimThreadCaches)
	{
		pas_scavenger_run_synchronously_now();
	}
	else
	{
		pas_scavenger_do_everything_except_remote_tlcs();
	}
}

void FMallocLibpas::OnPreFork()
{
	if (!FPlatformProcess::SupportsMultithreading())
	{
		check(pas_scavenger_should_suspend_count);
	}
}

void FMallocLibpas::OnPostFork()
{
	if (FForkProcessHelper::IsForkedMultithreadInstance())
	{
		pas_scavenger_resume();
	}
}

#endif // LIBPASMALLOC_ENABLED

