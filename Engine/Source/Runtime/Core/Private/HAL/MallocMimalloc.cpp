// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MallocMimalloc.cpp: MiMalloc
=============================================================================*/

#include "HAL/MallocMimalloc.h"

// Only use for supported platforms
#if MIMALLOC_ENABLED

#include "HAL/UnrealMemory.h"
#include "HAL/IConsoleManager.h"
#include "HAL/LowLevelMemTracker.h"
#include "Math/UnrealMathUtility.h"
#include "Thirdparty/IncludeMiMalloc.h"

#if PLATFORM_MAC
#include "Templates/AlignmentTemplates.h"
#endif

/** Value we fill a memory block with after it is free, in UE_BUILD_DEBUG **/
#define DEBUG_FILL_FREED (0xdd)

/** Value we fill a new memory block with, in UE_BUILD_DEBUG **/
#define DEBUG_FILL_NEW (0xcd)

// Dramatically reduce memory zeroing and page faults during alloc intense workloads
// by keeping freed pages for a little while instead of releasing them
// right away to the OS, effectively acting like a scratch buffer 
// until pages are both freed and inactive for the delay specified
// in milliseconds.
int32 GMiMallocMemoryResetDelay = 10000;

namespace UE::Mimalloc
{
	static void ApplyMemoryResetDelayChange(IConsoleVariable* InConsolveVariable = nullptr)
	{
		mi_option_set(mi_option_reset_delay, GMiMallocMemoryResetDelay);
	}

	static FAutoConsoleVariableRef CVarMiMallocMemoryResetDelay(
		TEXT("mi.MemoryResetDelay"),
		GMiMallocMemoryResetDelay,
		TEXT("The time in milliseconds to keep recently freed memory pages inside the process for reuse. This can dramatically reduce OS overhead of memory zeroing and page faults during alloc intense workloads."),
		FConsoleVariableDelegate::CreateStatic(&UE::Mimalloc::ApplyMemoryResetDelayChange),
		ECVF_Default
	);

#if ENABLE_LOW_LEVEL_MEM_TRACKER && MI_MALLOC_VERSION == 212
	// Called when OS memory was allocated.
	static void OnAlloc(void* Ptr, size_t Size, bool bIsLarge, bool bCommitted)
	{
		if (bCommitted)
		{
			LLM_PLATFORM_SCOPE(ELLMTag::FMalloc);
			LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelAlloc(ELLMTracker::Platform, Ptr, Size));
		}
	}

	// Called when OS memory was freed.
	static void OnFree(void* Ptr, size_t Size)
	{
		(void)Size;
		LLM_PLATFORM_SCOPE(ELLMTag::FMalloc);
		LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelFree(ELLMTracker::Platform, Ptr));
	}

	// Called when OS memory was committed.
	static void OnCommit(void* Ptr, size_t Size)
	{
		LLM_PLATFORM_SCOPE(ELLMTag::FMalloc);
		LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelAlloc(ELLMTracker::Platform, Ptr, Size));
	}

	// Called when OS memory was decommitted.
	static void OnDecommit(void* Ptr, size_t Size, bool /*bNeedsRecommit*/)
	{
		LLM_PLATFORM_SCOPE(ELLMTag::FMalloc);
		LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelFree(ELLMTracker::Platform, Ptr));
	}

	static void SetupTrackingIfAvailable()
	{
		mi_ext_os_tracking_t hooks {};
		hooks.on_alloc    = OnAlloc;
		hooks.on_free     = OnFree;
		hooks.on_commit   = OnCommit;
		hooks.on_decommit = OnDecommit;
		mi_ext_set_os_tracking(&hooks);
	}
#else
	static void SetupTrackingIfAvailable() {}
#endif
} // namespace UE::Mimalloc

FMallocMimalloc::FMallocMimalloc()
{
	FPlatformMemory::MiMallocInit();

	UE::Mimalloc::ApplyMemoryResetDelayChange();

#if AGGRESSIVE_MEMORY_SAVING
#	if MI_MALLOC_VERSION == 212
		mi_option_set(mi_option_deprecated_page_reset, 1);
		mi_option_set(mi_option_abandoned_page_purge, 1);
		mi_option_set(mi_option_deprecated_segment_reset, 1);
		mi_option_set(mi_option_purge_delay, 0);
#	else
		mi_option_set(mi_option_reset_decommits, 1);
		mi_option_set(mi_option_page_reset, 1);
		mi_option_set(mi_option_abandoned_page_reset, 1);
		mi_option_set(mi_option_segment_reset, 1);
		mi_option_set(mi_option_reset_delay, 0);
		mi_option_set(mi_option_segment_decommit_delay, 0);
#	endif

	mi_option_set(mi_option_eager_commit_delay, 4);
#endif

	UE::Mimalloc::SetupTrackingIfAvailable();
}

void* FMallocMimalloc::TryMalloc( SIZE_T Size, uint32 Alignment )
{
#if !UE_BUILD_SHIPPING
	uint64 LocalMaxSingleAlloc = MaxSingleAlloc.Load(EMemoryOrder::Relaxed);
	if (LocalMaxSingleAlloc != 0 && Size > LocalMaxSingleAlloc)
	{
		return nullptr;
	}
#endif

	void* NewPtr = nullptr;

	if (Alignment != DEFAULT_ALIGNMENT)
	{
		Alignment = FMath::Max(uint32(Size >= 16 ? 16 : 8), Alignment);
		NewPtr = mi_malloc_aligned(Size, Alignment);
	}
	else
	{
		NewPtr = mi_malloc_aligned(Size, uint32(Size >= 16 ? 16 : 8));
	}

#if UE_BUILD_DEBUG
	if (Size && NewPtr != nullptr)
	{
		FMemory::Memset(NewPtr, DEBUG_FILL_NEW, mi_usable_size(NewPtr));
	}
#endif

	return NewPtr;
}

void* FMallocMimalloc::Malloc(SIZE_T Size, uint32 Alignment)
{
	void* Result = TryMalloc(Size, Alignment);

	if (Result == nullptr && Size)
	{
		OutOfMemory(Size, Alignment);
	}

	return Result;
}

void* FMallocMimalloc::TryRealloc(void* Ptr, SIZE_T NewSize, uint32 Alignment)
{
#if !UE_BUILD_SHIPPING
	uint64 LocalMaxSingleAlloc = MaxSingleAlloc.Load(EMemoryOrder::Relaxed);
	if (LocalMaxSingleAlloc != 0 && NewSize > LocalMaxSingleAlloc)
	{
		return nullptr;
	}
#endif

#if UE_BUILD_DEBUG
	SIZE_T OldSize = 0;
	if (Ptr)
	{
		OldSize = mi_malloc_size(Ptr);
		if (NewSize < OldSize)
		{
			FMemory::Memset((uint8*)Ptr + NewSize, DEBUG_FILL_FREED, OldSize - NewSize); 
		}
	}
#endif
	void* NewPtr = nullptr;

	if (NewSize == 0)
	{
		mi_free(Ptr);

		return nullptr;
	}

#if PLATFORM_MAC
	// macOS expects all allocations to be aligned to 16 bytes, so on Mac we always have to use mi_realloc_aligned
	Alignment = AlignArbitrary(FMath::Max((uint32)16, Alignment), (uint32)16);
	NewPtr	= mi_realloc_aligned(Ptr, NewSize, Alignment);
#else
	if (Alignment != DEFAULT_ALIGNMENT)
	{
		Alignment = FMath::Max(NewSize >= 16 ? (uint32)16 : (uint32)8, Alignment);
		NewPtr = mi_realloc_aligned(Ptr, NewSize, Alignment);
	}
	else
	{
		NewPtr = mi_realloc(Ptr, NewSize);
	}
#endif
#if UE_BUILD_DEBUG
	if (NewPtr && NewSize > OldSize )
	{
		FMemory::Memset((uint8*)NewPtr + OldSize, DEBUG_FILL_NEW, mi_usable_size(NewPtr) - OldSize);
	}
#endif

	return NewPtr;
}

void* FMallocMimalloc::Realloc(void* Ptr, SIZE_T NewSize, uint32 Alignment)
{
	void* Result = TryRealloc(Ptr, NewSize, Alignment);

	if (Result == nullptr && NewSize)
	{
		OutOfMemory(NewSize, Alignment);
	}

	return Result;
}

void FMallocMimalloc::Free( void* Ptr )
{
	if( !Ptr )
	{
		return;
	}
#if UE_BUILD_DEBUG
	FMemory::Memset(Ptr, DEBUG_FILL_FREED, mi_usable_size(Ptr));
#endif
	mi_free(Ptr);
}

void* FMallocMimalloc::MallocZeroed(SIZE_T Size, uint32 Alignment)
{
	void* Result = TryMallocZeroed(Size, Alignment);

	if (Result == nullptr && Size)
	{
		OutOfMemory(Size, Alignment);
	}

	return Result;
}

void* FMallocMimalloc::TryMallocZeroed(SIZE_T Size, uint32 Alignment)
{
#if !UE_BUILD_SHIPPING
	uint64 LocalMaxSingleAlloc = MaxSingleAlloc.Load(EMemoryOrder::Relaxed);
	if (LocalMaxSingleAlloc != 0 && Size > LocalMaxSingleAlloc)
	{
		return nullptr;
	}
#endif

	void* NewPtr = nullptr;

	if (Alignment != DEFAULT_ALIGNMENT)
	{
		Alignment = FMath::Max(uint32(Size >= 16 ? 16 : 8), Alignment);
		NewPtr = mi_zalloc_aligned(Size, Alignment);
	}
	else
	{
		NewPtr = mi_zalloc_aligned(Size, uint32(Size >= 16 ? 16 : 8));
	}

	return NewPtr;
}

bool FMallocMimalloc::GetAllocationSize(void *Original, SIZE_T &SizeOut)
{
	SizeOut = mi_malloc_size(Original);
	return true;
}

void FMallocMimalloc::Trim(bool bTrimThreadCaches)
{
	mi_collect(bTrimThreadCaches);
}

#undef DEBUG_FILL_FREED
#undef DEBUG_FILL_NEW

#endif // MIMALLOC_ENABLED
