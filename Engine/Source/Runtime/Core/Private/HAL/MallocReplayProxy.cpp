// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
        MallocReplayProxy.cpp: Light-weight proxy that saves all memory 
        allocations in CSV format for later replay
=============================================================================*/

#include "HAL/MallocReplayProxy.h"

#if UE_USE_MALLOC_REPLAY_PROXY

#include "Misc/CString.h"
#include "HAL/PlatformProcess.h"

struct FMallocReplayProxyCloserOnExit
{
	static FMallocReplayProxy* InstanceToClose;

	~FMallocReplayProxyCloserOnExit()
	{
		if (InstanceToClose)
		{
			InstanceToClose->CloseHistory();
		}
	}
}
MallocReplayProxyCloserOnExit;

FMallocReplayProxy* FMallocReplayProxyCloserOnExit::InstanceToClose = nullptr;

FMallocReplayProxy::FMallocReplayProxy(FMalloc* InMalloc)
	: UsedMalloc(InMalloc)
	, CurrentCacheIdx(0)
	, OperationNumber(0)
{
	checkf(UsedMalloc, TEXT("FMallocReplayProxy is used without a valid malloc!"));

	char Buf[128];
	FCStringAnsi::Sprintf(Buf, "mallocreplay-pid-%d.txt", FPlatformProcess::GetCurrentProcessId());
	HistoryFile = fopen(Buf, "wb");
	// if it is null, we will silenty ignore saves
	if (HistoryFile)
	{
		fprintf(HistoryFile, "Operation ResultPointer PointerIn SizeIn AlignmentIn\n");

		// GMalloc may not be destroyed, close history on exit ourselves
		MallocReplayProxyCloserOnExit.InstanceToClose = this;
	}
}

FMallocReplayProxy::~FMallocReplayProxy()
{
	CloseHistory();
}

void FMallocReplayProxy::CloseHistory()
{
	FScopeLock Lock(&HistoryLock);

	DumpHistoryToDisk();
	if (HistoryFile)
	{
		fprintf(HistoryFile, "\nGracefully closed\n");
		fclose(HistoryFile);
		HistoryFile = nullptr;
	}
}

void FMallocReplayProxy::DumpHistoryToDisk()
{
	if (LIKELY(HistoryFile))
	{
		for (int32 Idx = 0; Idx < CurrentCacheIdx; ++Idx)
		{
			fprintf(HistoryFile, "%s %llu %llu %llu %u\t# %llu\n", HistoryCache[Idx].Operation, (uint64)(HistoryCache[Idx].PointerOut), (uint64)(HistoryCache[Idx].PointerIn), (uint64)HistoryCache[Idx].Size, HistoryCache[Idx].Alignment, ++OperationNumber);
		}
	}

	CurrentCacheIdx = 0;
}

void FMallocReplayProxy::InitializeStatsMetadata()
{
	UsedMalloc->InitializeStatsMetadata();
}

void* FMallocReplayProxy::Malloc(SIZE_T Size, uint32 Alignment)
{
	void* Result = UsedMalloc->Malloc(Size, Alignment);
	AddToHistory("Malloc", Result, nullptr, Size, Alignment);
	return Result;
}

void* FMallocReplayProxy::Realloc(void* Ptr, SIZE_T NewSize, uint32 Alignment)
{
	void* Result = UsedMalloc->Realloc(Ptr, NewSize, Alignment);
	AddToHistory("Realloc", Result, Ptr, NewSize, Alignment);
	return Result;
}

void FMallocReplayProxy::Free(void* Ptr)
{
	if (LIKELY(Ptr))
	{
		UsedMalloc->Free(Ptr);
		AddToHistory("Free", nullptr, Ptr, 0, 0);
	}
}

SIZE_T FMallocReplayProxy::QuantizeSize(SIZE_T Count, uint32 Alignment)
{
	return UsedMalloc->QuantizeSize(Count, Alignment);
}

void FMallocReplayProxy::UpdateStats()
{
	UsedMalloc->UpdateStats();
}

void FMallocReplayProxy::GetAllocatorStats(FGenericMemoryStats& out_Stats)
{
	UsedMalloc->GetAllocatorStats(out_Stats);
}

void FMallocReplayProxy::DumpAllocatorStats(class FOutputDevice& Ar)
{
	UsedMalloc->DumpAllocatorStats(Ar);
}

bool FMallocReplayProxy::IsInternallyThreadSafe() const
{
	return UsedMalloc->IsInternallyThreadSafe();
}

bool FMallocReplayProxy::ValidateHeap()
{
	return UsedMalloc->ValidateHeap();
}

bool FMallocReplayProxy::Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	return UsedMalloc->Exec(InWorld, Cmd, Ar);
}

bool FMallocReplayProxy::GetAllocationSize(void *Original, SIZE_T &SizeOut)
{
	return UsedMalloc->GetAllocationSize(Original, SizeOut);
}

const TCHAR* FMallocReplayProxy::GetDescriptiveName()
{
	return UsedMalloc->GetDescriptiveName();
}

void FMallocReplayProxy::Trim(bool bTrimThreadCaches)
{
	UsedMalloc->Trim(bTrimThreadCaches);
}

void FMallocReplayProxy::SetupTLSCachesOnCurrentThread()
{
	UsedMalloc->SetupTLSCachesOnCurrentThread();
}

void FMallocReplayProxy::ClearAndDisableTLSCachesOnCurrentThread()
{
	UsedMalloc->ClearAndDisableTLSCachesOnCurrentThread();
}

#endif // UE_USE_MALLOC_REPLAY_PROXY
