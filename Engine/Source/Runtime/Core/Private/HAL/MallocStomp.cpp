// Copyright Epic Games, Inc. All Rights Reserved.

#include "HAL/MallocStomp.h"

#include "GenericPlatform/GenericPlatformMemory.h"
#include "HAL/IConsoleManager.h"
#include "HAL/UnrealMemory.h"
#include "Hash/xxhash.h"
#include "Math/UnrealMathUtility.h"

#if PLATFORM_UNIX || PLATFORM_MAC
	#include <sys/mman.h>
#endif

#if WITH_MALLOC_STOMP

#if PLATFORM_WINDOWS
// MallocStomp can keep virtual address range reserved after memory block is freed, while releasing the physical memory.
// This dramatically increases accuracy of use-after-free detection, but consumes significant amount of memory for the OS page table.
// Virtual memory limit for a process on Win10 is 128 TB, which means we can afford to keep virtual memory reserved for a very long time.
// Running Infiltrator demo consumes ~700MB of virtual address space per second.
#define MALLOC_STOMP_KEEP_VIRTUAL_MEMORY 1
thread_local UPTRINT VirtualAddressCursor = 0;
thread_local SIZE_T VirtualAddressMax = 0;
#else
#define MALLOC_STOMP_KEEP_VIRTUAL_MEMORY 0
#endif

// 64-bit ABIs on x86_64 expect a 16-byte alignment
#define STOMPALIGNMENT 16U

static void MallocStompOverrunTest()
{
#if !USING_CODE_ANALYSIS
	const uint32 ArraySize = 4;
	uint8* Pointer = new uint8[ArraySize];
	// Overrun.
	Pointer[ArraySize+1+ STOMPALIGNMENT] = 0;
#endif // !USING_CODE_ANALYSIS
}

FAutoConsoleCommand MallocStompTestCommand
(
	TEXT( "MallocStomp.OverrunTest" ),
	TEXT( "Overrun test for the FMallocStomp" ),
	FConsoleCommandDelegate::CreateStatic( &MallocStompOverrunTest )
);

struct FMallocStomp::FAllocationData
{
	/** Pointer to the full allocation. Needed so the OS knows what to free. */
	void*	FullAllocationPointer;
	/** Full size of the allocation including the extra page. */
	SIZE_T	FullSize;
	/** Size of the allocation requested. */
	SIZE_T	Size;
	/** Sentinel used to check for underrun. */
	SIZE_T	Sentinel;

	/** Calculate the expected sentinel value for this allocation data. */
	SIZE_T CalculateSentinel() const
	{
		return (SIZE_T)FXxHash64::HashBuffer(this, offsetof(FAllocationData, Sentinel)).Hash;
	}
};

FMallocStomp::FMallocStomp(const bool InUseUnderrunMode) 
	: PageSize(FPlatformMemory::GetConstants().PageSize)
	, bUseUnderrunMode(InUseUnderrunMode)
{
}

void* FMallocStomp::Malloc(SIZE_T Size, uint32 Alignment)
{
	void* Result = TryMalloc(Size, Alignment);

	if (Result == nullptr)
	{
		FPlatformMemory::OnOutOfMemory(Size, Alignment);
	}

	return Result;
}

void* FMallocStomp::TryMalloc(SIZE_T Size, uint32 Alignment)
{
	if (Size == 0U)
	{
		Size = 1U;
	}

	// 64-bit ABIs on x86_64 expect a 16-byte alignment
	Alignment = FMath::Max<uint32>(Alignment, STOMPALIGNMENT);

	constexpr static SIZE_T AllocationDataSize = sizeof(FAllocationData);

	const SIZE_T AlignedSize = ((Size + Alignment - 1) & -(int32)Alignment);
	const SIZE_T AlignmentSize = Alignment > PageSize ? Alignment - PageSize : 0;
	const SIZE_T AllocFullPageSize = (AlignedSize + AlignmentSize + AllocationDataSize + PageSize - 1) & -(SSIZE_T)PageSize;
	const SIZE_T TotalAllocationSize = AllocFullPageSize + PageSize;

#if PLATFORM_UNIX || PLATFORM_MAC
	void* FullAllocationPointer = mmap(nullptr, TotalAllocationSize, PROT_READ | PROT_WRITE, MAP_PRIVATE | MAP_ANON, -1, 0);
#elif PLATFORM_WINDOWS && MALLOC_STOMP_KEEP_VIRTUAL_MEMORY
	// Allocate virtual address space from current block using linear allocation strategy.
	// If there is not enough space, try to allocate new block from OS. Report OOM if block allocation fails.
	void* FullAllocationPointer = nullptr;
	if (VirtualAddressCursor + TotalAllocationSize <= VirtualAddressMax)
	{
		FullAllocationPointer = (void*)(VirtualAddressCursor);
	}
	else
	{
		const SIZE_T ReserveSize = FMath::Max(VirtualAddressBlockSize, TotalAllocationSize);

		// Reserve a new block of virtual address space that will be linearly sub-allocated
		// We intentionally don't keep track of reserved blocks, as we never need to explicitly release them.
		FullAllocationPointer = VirtualAlloc(nullptr, ReserveSize, MEM_RESERVE, PAGE_NOACCESS);

		VirtualAddressCursor = UPTRINT(FullAllocationPointer);
		VirtualAddressMax = VirtualAddressCursor + ReserveSize;
	}

	// No atomics as this is a thread_local
	VirtualAddressCursor += TotalAllocationSize;

#else
	void* FullAllocationPointer = FPlatformMemory::BinnedAllocFromOS(TotalAllocationSize);
#endif // PLATFORM_UNIX || PLATFORM_MAC

	if (!FullAllocationPointer)
	{
		return nullptr;
	}

	void* ReturnedPointer = nullptr;

	checkSlow(IsAligned(FullAllocationPointer, PageSize));

	if (bUseUnderrunMode)
	{
		ReturnedPointer = Align((uint8*)FullAllocationPointer + PageSize + AllocationDataSize, Alignment);
		void* AllocDataPointerStart = static_cast<FAllocationData*>(ReturnedPointer) - 1;
		checkSlow(AllocDataPointerStart >= FullAllocationPointer);

#if PLATFORM_WINDOWS && MALLOC_STOMP_KEEP_VIRTUAL_MEMORY
		// Commit physical pages to the used range, leaving the first page unmapped.
		void* CommittedMemory = VirtualAlloc(AllocDataPointerStart, AllocationDataSize + AlignedSize, MEM_COMMIT, PAGE_READWRITE);
		if (!CommittedMemory)
		{
			// Failed to allocate and commit physical memory pages.
			return nullptr;
		}
		check(CommittedMemory == AlignDown(AllocDataPointerStart, PageSize));
#else
		// Page protect the first page, this will cause the exception in case there is an underrun.
		FPlatformMemory::PageProtect((uint8*)AlignDown(AllocDataPointerStart, PageSize) - PageSize, PageSize, false, false);
#endif
	} //-V773
	else
	{
		ReturnedPointer = AlignDown((uint8*)FullAllocationPointer + AllocFullPageSize - AlignedSize, Alignment);
		void* ReturnedPointerEnd = (uint8*)ReturnedPointer + AlignedSize;
		checkSlow(IsAligned(ReturnedPointerEnd, PageSize));

		void* AllocDataPointerStart = static_cast<FAllocationData*>(ReturnedPointer) - 1;
		checkSlow(AllocDataPointerStart >= FullAllocationPointer);

#if PLATFORM_WINDOWS && MALLOC_STOMP_KEEP_VIRTUAL_MEMORY
		// Commit physical pages to the used range, leaving the last page unmapped.
		void* CommitPointerStart = AlignDown(AllocDataPointerStart, PageSize);
		void* CommittedMemory = VirtualAlloc(CommitPointerStart, SIZE_T((uint8*)ReturnedPointerEnd - (uint8*)CommitPointerStart), MEM_COMMIT, PAGE_READWRITE);
		if (!CommittedMemory)
		{
			// Failed to allocate and commit physical memory pages.
			return nullptr;
		}
		check(CommittedMemory == CommitPointerStart);
#else
		// Page protect the last page, this will cause the exception in case there is an overrun.
		FPlatformMemory::PageProtect(ReturnedPointerEnd, PageSize, false, false);
#endif
	} //-V773

	checkSlow(IsAligned(FullAllocationPointer, PageSize));
	checkSlow(IsAligned(TotalAllocationSize, PageSize));
	checkSlow(IsAligned(ReturnedPointer, Alignment));
	checkSlow((uint8*)ReturnedPointer + AlignedSize <= (uint8*)FullAllocationPointer + TotalAllocationSize);

	FAllocationData& AllocationData = static_cast<FAllocationData*>(ReturnedPointer)[-1];
	AllocationData = { FullAllocationPointer, TotalAllocationSize, AlignedSize, 0 };
	AllocationData.Sentinel = AllocationData.CalculateSentinel();

	return ReturnedPointer;
}

void* FMallocStomp::Realloc(void* InPtr, SIZE_T NewSize, uint32 Alignment)
{
	void* Result = TryRealloc(InPtr, NewSize, Alignment);

	if (Result == nullptr && NewSize)
	{
		FPlatformMemory::OnOutOfMemory(NewSize, Alignment);
	}

	return Result;
}

void* FMallocStomp::TryRealloc(void* InPtr, SIZE_T NewSize, uint32 Alignment)
{
	if (NewSize == 0U)
	{
		Free(InPtr);
		return nullptr;
	}

	void* ReturnPtr = nullptr;

	if (InPtr != nullptr)
	{
		ReturnPtr = TryMalloc(NewSize, Alignment);

		if (ReturnPtr != nullptr)
		{
			FAllocationData* AllocDataPtr = reinterpret_cast<FAllocationData*>(reinterpret_cast<uint8*>(InPtr) - sizeof(FAllocationData));
			FMemory::Memcpy(ReturnPtr, InPtr, FMath::Min(AllocDataPtr->Size, NewSize));
			Free(InPtr);
		}
	}
	else
	{
		ReturnPtr = TryMalloc(NewSize, Alignment);
	}

	return ReturnPtr;
}

void FMallocStomp::Free(void* InPtr)
{
	if(InPtr == nullptr)
	{
		return;
	}

	FAllocationData *AllocDataPtr = reinterpret_cast<FAllocationData*>(InPtr);
	AllocDataPtr--;

	// Check the sentinel to verify that the allocation data is intact.
	if (AllocDataPtr->Sentinel != AllocDataPtr->CalculateSentinel())
	{
		// There was a memory underrun related to this allocation.
		UE_DEBUG_BREAK();
	}

#if PLATFORM_UNIX || PLATFORM_MAC
	munmap(AllocDataPtr->FullAllocationPointer, AllocDataPtr->FullSize);
#elif PLATFORM_WINDOWS && MALLOC_STOMP_KEEP_VIRTUAL_MEMORY
	// Unmap physical memory, but keep virtual address range reserved to catch use-after-free errors.
	#if USING_CODE_ANALYSIS
	MSVC_PRAGMA(warning(push))
	MSVC_PRAGMA(warning(disable : 6250)) // Suppress C6250, as virtual address space is "leaked" by design.
	#endif

	VirtualFree(AllocDataPtr->FullAllocationPointer, AllocDataPtr->FullSize, MEM_DECOMMIT);

	#if USING_CODE_ANALYSIS
	MSVC_PRAGMA(warning(pop))
	#endif
#else
	FPlatformMemory::BinnedFreeToOS(AllocDataPtr->FullAllocationPointer, AllocDataPtr->FullSize);
#endif // PLATFORM_UNIX || PLATFORM_MAC
}

bool FMallocStomp::GetAllocationSize(void *Original, SIZE_T &SizeOut) 
{
	if(Original == nullptr)
	{
		SizeOut = 0U;
	}
	else
	{
		FAllocationData *AllocDataPtr = reinterpret_cast<FAllocationData*>(Original);
		AllocDataPtr--;
		SizeOut = AllocDataPtr->Size;
	}

	return true;
}

#endif // WITH_MALLOC_STOMP
