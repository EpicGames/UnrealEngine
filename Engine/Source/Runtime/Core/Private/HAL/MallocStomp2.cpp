// Copyright Epic Games, Inc. All Rights Reserved.

#include "HAL/MallocStomp2.h"

#if WITH_MALLOC_STOMP2
#include "Math/UnrealMathUtility.h"
#include "Misc/Parse.h"
#include "HAL/UnrealMemory.h"
#include "HAL/IConsoleManager.h"
#include "GenericPlatform/GenericPlatformMemory.h"
#include "Algo/BinarySearch.h"
#include "Templates/AlignmentTemplates.h"

CORE_API FMallocStomp2* GMallocStomp2 = nullptr;
CORE_API bool GMallocStomp2Enabled = false;


uint32 FMallocStomp2::DisableTlsSlot = FPlatformTLS::InvalidTlsSlot;
SIZE_T FMallocStomp2::MinSize = 0;
SIZE_T FMallocStomp2::MaxSize = 0;

// MallocStomp2 keeps virtual address ranges reserved after a memory block is freed, while releasing the physical memory.
// This dramatically increases accuracy of use-after-free detection, but consumes a significant amount of memory for the OS page table.
// Virtual memory limit for a process on Win10 is 128 TB, which means we can afford to keep virtual memory reserved for a very long time.
// Running Infiltrator demo consumes ~700MB of virtual address space per second.

static void MallocStomp2OverrunTest()
{
#if !USING_CODE_ANALYSIS
	const uint32 ArraySize = 16;
	uint8* Pointer = new uint8[ArraySize];
	// Overrun.
	Pointer[ArraySize+1] = 0;
#endif // !USING_CODE_ANALYSIS
}

FAutoConsoleCommand MallocStomp2TestCommand
(
	TEXT( "MallocStomp2.OverrunTest" ),
	TEXT( "Overrun test for the FMallocStomp2" ),
	FConsoleCommandDelegate::CreateStatic( &MallocStomp2OverrunTest )
);

FMallocStomp2::FMallocStomp2(FMalloc* InMalloc, const bool InUseUnderrunMode)
	: PageSize(FPlatformMemory::GetConstants().PageSize)
	, UsedMalloc(InMalloc)
	, bUseUnderrunMode(InUseUnderrunMode)
{
}

void* FMallocStomp2::Malloc(SIZE_T Size, uint32 Alignment)
{
	void* Result = TryMalloc(Size, Alignment);

	if (Result == nullptr)
	{
		FPlatformMemory::OnOutOfMemory(Size, Alignment);
	}

	return Result;
}

void* FMallocStomp2::TryMalloc(SIZE_T Size, uint32 Alignment)
{
	// 64-bit ABIs on x86_64 expect a 16-byte alignment
	Alignment = FMath::Max<uint32>(Alignment, 16U);

	void* FullAllocationPointer = nullptr;

	if (Size < MinSize || Size >= MaxSize || Alignment > PageSize || FPlatformTLS::GetTlsValue(FMallocStomp2::DisableTlsSlot) != 0)
	{
		return UsedMalloc->Malloc(Size, Alignment);
	}

	const SIZE_T AlignedSize = Align(Size, Alignment);
	const SIZE_T AllocFullPageSize = Align(AlignedSize + sizeof(FAllocationData), PageSize);
	const SIZE_T TotalAllocationSize = AllocFullPageSize + PageSize;


	int64 LocalCurrentAddressSpaceStompDataBlockIndex = CurrentAddressSpaceStompDataBlockIndex;
	while (FullAllocationPointer == nullptr && LocalCurrentAddressSpaceStompDataBlockIndex < MaxAddressSpaceStompDataBlocks)
	{
		FullAllocationPointer = AddressSpaceStompDataBlocks[LocalCurrentAddressSpaceStompDataBlockIndex].Malloc(TotalAllocationSize, PageSize, bUseUnderrunMode);
		if (FullAllocationPointer == nullptr)
		{
			int64 WantedLocalCurrentAddressSpaceStompDataBlockIndex = LocalCurrentAddressSpaceStompDataBlockIndex + 1;
			if (CurrentAddressSpaceStompDataBlockIndex.compare_exchange_weak(LocalCurrentAddressSpaceStompDataBlockIndex, WantedLocalCurrentAddressSpaceStompDataBlockIndex))
			{
				LocalCurrentAddressSpaceStompDataBlockIndex = WantedLocalCurrentAddressSpaceStompDataBlockIndex;
			}
		}
	}

	if (!FullAllocationPointer)
	{
		return nullptr;
	}

	void* ReturnedPointer = nullptr;
	static const SIZE_T AllocationDataSize = sizeof(FAllocationData);

	if(bUseUnderrunMode)
	{
		const SIZE_T AlignedAllocationData = Align(AllocationDataSize, Alignment);
		ReturnedPointer = reinterpret_cast<void*>(reinterpret_cast<uint8*>(FullAllocationPointer) + PageSize + AlignedAllocationData);
	}
	else
	{
		ReturnedPointer = reinterpret_cast<void*>(reinterpret_cast<uint8*>(FullAllocationPointer) + AllocFullPageSize - AlignedSize);

	}

	FAllocationData* AllocDataPointer = reinterpret_cast<FAllocationData*>(reinterpret_cast<uint8*>(ReturnedPointer) - AllocationDataSize);
	AllocDataPointer->FullAllocationPointer = FullAllocationPointer;
	AllocDataPointer->FullSize = TotalAllocationSize;
	AllocDataPointer->Size = Size;
	AllocDataPointer->Sentinel = SentinelExpectedValue;

	memset(ReturnedPointer, 0, Size);

	return ReturnedPointer;
}

void* FMallocStomp2::Realloc(void* InPtr, SIZE_T NewSize, uint32 Alignment)
{
	void* Result = TryRealloc(InPtr, NewSize, Alignment);

	if (Result == nullptr && NewSize)
	{
		FPlatformMemory::OnOutOfMemory(NewSize, Alignment);
	}

	return Result;
}

void* FMallocStomp2::TryRealloc(void* InPtr, SIZE_T NewSize, uint32 Alignment)
{
	if (NewSize == 0U)
	{
		Free(InPtr);
		return nullptr;
	}

	void* ReturnPtr = nullptr;

	if (InPtr != nullptr)
	{
		ReturnPtr = TryMalloc(NewSize, Alignment);

		if (ReturnPtr != nullptr)
		{
			SIZE_T OldSize = 0;
			GetAllocationSize(InPtr, OldSize);
			FMemory::Memcpy(ReturnPtr, InPtr, FMath::Min(OldSize, NewSize));
			Free(InPtr);
		}
	}
	else
	{
		ReturnPtr = TryMalloc(NewSize, Alignment);
	}
	return ReturnPtr;
}

void FMallocStomp2::Free(void* InPtr)
{
	if(InPtr == nullptr)
	{
		return;
	}

	int Index = IsPartOf(InPtr);
	if (Index >= 0)
	{
		// we own it, we can free it
		//Get the actual address space index, the one we got from IsPartOf is just the 1st one in a run of contiguous address space
		Index = FindAddressSpaceStompDataBlockIndex(InPtr, AddressSpaceStompDataBlocksRangeEntries[Index].StartIndex);

		FAllocationData* AllocDataPtr = reinterpret_cast<FAllocationData*>(InPtr);
		AllocDataPtr--;
		// Check that our sentinel is intact.
		if (AllocDataPtr->Sentinel != SentinelExpectedValue)
		{
			// There was a memory underrun related to this allocation.
			UE_DEBUG_BREAK();
		}

		AddressSpaceStompDataBlocks[Index].Free(AllocDataPtr->FullAllocationPointer, AllocDataPtr->FullSize-PageSize);
	}
	else
	{
		UsedMalloc->Free(InPtr);
	}
}

bool FMallocStomp2::GetAllocationSize(void * InPtr, SIZE_T &SizeOut)
{
	if(InPtr == nullptr)
	{
		SizeOut = 0U;
	}
	else
	{

		int Index = IsPartOf(InPtr);
		if (Index >= 0)
		{
			FAllocationData* AllocDataPtr = reinterpret_cast<FAllocationData*>(InPtr);
			AllocDataPtr--;
			SizeOut = AllocDataPtr->Size;
		}
		else
		{
			return UsedMalloc->GetAllocationSize(InPtr, SizeOut);
		}
	}

	return true;
}

static int32 CDECL qsort_compare_FAddressSpaceStompDataBlock(const void* A, const void* B)
{
	const FMallocStomp2::FAddressSpaceStompDataBlock* Element1 = (const FMallocStomp2::FAddressSpaceStompDataBlock*)A;
	const FMallocStomp2::FAddressSpaceStompDataBlock* Element2 = (const FMallocStomp2::FAddressSpaceStompDataBlock*)B;
	if (Element1->StartAddress < Element2->StartAddress)
	{
		return -1;
	}
	return 1;
}


int FORCENOINLINE FMallocStomp2::IsPartOf(void* InPtr)
{
	UPTRINT Ptr = (UPTRINT)InPtr;

#if 0
	for (int BlockRangeIndex = 0; BlockRangeIndex < NumberOfRangeEntries; ++BlockRangeIndex)
	{
		if (AddressSpaceStompDataBlocksRangeEntries[BlockRangeIndex].StartIndex > CurrentAddressSpaceStompDataBlockIndex)
		{
			return -1;
		}
		if (Ptr >= AddressSpaceStompDataBlocksRangeEntries[BlockRangeIndex].StartAddress && Ptr < AddressSpaceStompDataBlocksRangeEntries[BlockRangeIndex].EndAddress)
		{
			return AddressSpaceStompDataBlocksRangeEntries[BlockRangeIndex].StartIndex;
		}
	}
#endif

	// Binary chop our way through the list of block ranges
	int Offset = NumberOfRangeEntries;
	int Start = 0;
	while (Offset > 0 )
	{

		int LeftoverSize = Offset % 2;
		Offset /= 2;

		const int32 CheckIndex = Start + Offset;

		if (Ptr >= AddressSpaceStompDataBlocksRangeEntries[CheckIndex].EndAddress)
		{
			Start = CheckIndex + LeftoverSize;
		}
		else if (Ptr >= AddressSpaceStompDataBlocksRangeEntries[CheckIndex].StartAddress)
		{
			// found a match
			return CheckIndex;
		}
	}
	return -1;
}

int FMallocStomp2::FindAddressSpaceStompDataBlockIndex(void* InPtr, int Index)
{
	UPTRINT Ptr = (UPTRINT)InPtr;
	for (; Index < MaxAddressSpaceStompDataBlocks; ++Index)
	{
		if ( Ptr < AddressSpaceStompDataBlocks[Index].EndAddress )
		{
			break;
		}
	}
	return Index;
}

FORCENOINLINE void FMallocStomp2::Init()
{
	DisableTlsSlot = FPlatformTLS::AllocTlsSlot();

	CurrentAddressSpaceStompDataBlockIndex = 0;
	SIZE_T AddressSpaceReserved = 0;
	int InitializedBlocks = 0;
	for ( ; InitializedBlocks < MaxAddressSpaceStompDataBlocks; ++InitializedBlocks)
	{
		if (AddressSpaceStompDataBlocks[InitializedBlocks].Init())
		{
			AddressSpaceReserved += AddressSpaceStompDataBlocks[InitializedBlocks].Size;
			if (AddressSpaceReserved >= TargetAddressSpaceToReserve)
			{
				break;
			}
		}
		else
		{
			break;
		}
	}

	for (int i = InitializedBlocks; i < MaxAddressSpaceStompDataBlocks; ++i)
	{
		memset(&AddressSpaceStompDataBlocks[i], 0, sizeof(AddressSpaceStompDataBlocks[i]));
	}

	// Sort AddressSpaceStompDataBlocks by increasing start address
	// We use qsort rather than Algo::sort because we have an atomic in the address space block, qsort moves things with memcpy, Algo::Sort uses move operators which are explicitly removed from atomic items
	// Its ok though as on most platforms the atomic is just a naked uint64, on some platforms it might well have locking objects, which can't be copied.
	qsort(&AddressSpaceStompDataBlocks, InitializedBlocks, sizeof(FAddressSpaceStompDataBlock), qsort_compare_FAddressSpaceStompDataBlock);


	NumberOfRangeEntries = 0;

	// to help search the address spaces we store a list of contiguous runs
	for (int Index = 0; Index < InitializedBlocks; )
	{
		int EndOfRunIndex = Index;
		while (EndOfRunIndex <= MaxAddressSpaceStompDataBlocks-1)
		{
			if (AddressSpaceStompDataBlocks[EndOfRunIndex].EndAddress == AddressSpaceStompDataBlocks[EndOfRunIndex+1].StartAddress)
			{
				// Memory is contiguous
				EndOfRunIndex++;
			}
			else
			{
				break;
			}
		}

		AddressSpaceStompDataBlocksRangeEntries[NumberOfRangeEntries].StartAddress = AddressSpaceStompDataBlocks[Index].StartAddress;
		AddressSpaceStompDataBlocksRangeEntries[NumberOfRangeEntries].EndAddress = AddressSpaceStompDataBlocks[EndOfRunIndex].EndAddress;
		AddressSpaceStompDataBlocksRangeEntries[NumberOfRangeEntries].StartIndex = Index;
		AddressSpaceStompDataBlocksRangeEntries[NumberOfRangeEntries].EndIndex = EndOfRunIndex;
		NumberOfRangeEntries++;
		Index = EndOfRunIndex + 1;
	}

}

FMalloc* FMallocStomp2::OverrideIfEnabled(FMalloc* InUsedAlloc)
{
	if (GMallocStomp2Enabled)
	{
#if !UE_BUILD_SHIPPING
		const TCHAR* CommandLine = ::GetCommandLineW();
		FParse::Value(CommandLine, TEXT("MallocStomp2MinSize="), MinSize);
		FParse::Value(CommandLine, TEXT("MallocStomp2MaxSize="), MaxSize);
		bool bUseUnderrunMode = FParse::Param(CommandLine, TEXT("MallocStomp2UnderrunMode"));
		ensureMsgf(MaxSize > 0, TEXT("MallocStomp2 has been enabled but MallocStomp2MaxSize is set to 0 - this will not produce meaningful results."));
		ensureMsgf(MaxSize >= MinSize, TEXT("MallocStomp2 has been enabled but MallocStomp2MaxSize is smaller than MallocStomp2MinSize - this will not produce meaningful results."));
#else
		bool bUseUnderrunMode = false;
#endif
		GMallocStomp2 = new FMallocStomp2(InUsedAlloc, bUseUnderrunMode);
		GMallocStomp2->Init();
		return GMallocStomp2;
	}
	return InUsedAlloc;
}

static void EnableMallocStomp2(const TArray<FString>& Args)
{
	GMallocStomp2Enabled = true;
}
FAutoConsoleCommand GMallocStomp2Enable(TEXT("MallocStomp2.Enable"), TEXT("Enable MallocStomp2"), FConsoleCommandWithArgsDelegate::CreateStatic(EnableMallocStomp2));

static void DisableMallocStomp2(const TArray<FString>& Args)
{
	GMallocStomp2Enabled = false;
}
FAutoConsoleCommand GMallocStomp2Disable(TEXT("MallocStomp2.Disable"), TEXT("Disable MallocStomp2"), FConsoleCommandWithArgsDelegate::CreateStatic(DisableMallocStomp2));

static void MallocStomp2SetMinSize(const TArray<FString>& Args)
{
	if (GMallocStomp2 != nullptr && Args.Num())
	{
		GMallocStomp2->MinSize = FCString::Atoi64(*Args[0]);
	}
}
FAutoConsoleCommand GMallocStomp2SetMinSize(TEXT("MallocStomp2.MinSize"), TEXT("Set the minimum size MallocStomp2 should track"), FConsoleCommandWithArgsDelegate::CreateStatic(MallocStomp2SetMinSize));

static void MallocStomp2SetMaxSize(const TArray<FString>& Args)
{
	if (GMallocStomp2 != nullptr && Args.Num())
	{
		GMallocStomp2->MaxSize = FCString::Atoi64(*Args[0]);
	}
}
FAutoConsoleCommand GMallocStomp2SetMaxSize(TEXT("MallocStomp2.MaxSize"), TEXT("Set the maximum size MallocStomp2 should track"), FConsoleCommandWithArgsDelegate::CreateStatic(MallocStomp2SetMaxSize));

#endif //WITH_MALLOC_STOMP2