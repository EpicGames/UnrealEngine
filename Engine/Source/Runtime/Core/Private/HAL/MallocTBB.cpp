// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MallocTTB.cpp: IntelTTB Malloc
=============================================================================*/

#include "HAL/MallocTBB.h"
#include "Math/UnrealMathUtility.h"
#include "HAL/UnrealMemory.h"
#if PLATFORM_MAC
#include "Templates/AlignmentTemplates.h"
#endif

// Only use for supported platforms
#if TBBMALLOC_ENABLED

/** Value we fill a memory block with after it is free, in UE_BUILD_DEBUG **/
#define DEBUG_FILL_FREED (0xdd)

/** Value we fill a new memory block with, in UE_BUILD_DEBUG **/
#define DEBUG_FILL_NEW (0xcd)

// Statically linked tbbmalloc requires tbbmalloc_debug.lib in debug
#if UE_BUILD_DEBUG && !defined(NDEBUG)	// Use !defined(NDEBUG) to check to see if we actually are linking with Debug third party libraries (bDebugBuildsActuallyUseDebugCRT)
	#ifndef TBB_USE_DEBUG
		#define TBB_USE_DEBUG 1
	#endif
#endif
THIRD_PARTY_INCLUDES_START
#include <tbb/scalable_allocator.h>
THIRD_PARTY_INCLUDES_END

void* FMallocTBB::TryMalloc( SIZE_T Size, uint32 Alignment )
{
#if !UE_BUILD_SHIPPING
	uint64 LocalMaxSingleAlloc = MaxSingleAlloc.Load(EMemoryOrder::Relaxed);
	if (LocalMaxSingleAlloc != 0 && Size > LocalMaxSingleAlloc)
	{
		return nullptr;
	}
#endif

	void* NewPtr = nullptr;
#if PLATFORM_MAC
	// macOS expects all allocations to be aligned to 16 bytes, but TBBs default alignment is 8, so on Mac we always have to use scalable_aligned_malloc.
	// Contrary to scalable_malloc, scalable_aligned_malloc returns nullptr when trying to allocate 0 bytes, which is inconsistent with system malloc, so
	// for 0 bytes requests we actually allocate sizeof(size_t), which is exactly what scalable_malloc does internally in such case.
	// scalable_aligned_realloc and scalable_realloc behave the same in this regard, so this is only needed here.
	Alignment = FMath::Max((uint32)16, Alignment);
	NewPtr = scalable_aligned_malloc(Size ? Size : sizeof(size_t), Alignment);
#else
	if( Alignment != DEFAULT_ALIGNMENT )
	{
		Alignment = FMath::Max(Size >= 16 ? (uint32)16 : (uint32)8, Alignment);
		NewPtr = scalable_aligned_malloc( Size, Alignment );
	}
	else
	{
		// Fulfill the promise of DEFAULT_ALIGNMENT, which aligns 16-byte or larger structures to 16 bytes,
		// while TBB aligns to 8 by default.
		NewPtr = scalable_aligned_malloc( Size, Size >= 16 ? (uint32)16 : (uint32)8);
	}
#endif

#if UE_BUILD_DEBUG
	if (Size && NewPtr != nullptr)
	{
		FMemory::Memset(NewPtr, DEBUG_FILL_NEW, scalable_msize(NewPtr));
	}
#endif

	return NewPtr;
}

void* FMallocTBB::Malloc(SIZE_T Size, uint32 Alignment)
{
	void* Result = TryMalloc(Size, Alignment);

	if (Result == nullptr && Size)
	{
		OutOfMemory(Size, Alignment);
	}

	return Result;
}

void* FMallocTBB::TryRealloc( void* Ptr, SIZE_T NewSize, uint32 Alignment )
{
#if !UE_BUILD_SHIPPING
	uint64 LocalMaxSingleAlloc = MaxSingleAlloc.Load(EMemoryOrder::Relaxed);
	if (LocalMaxSingleAlloc != 0 && NewSize > LocalMaxSingleAlloc)
	{
		return nullptr;
	}
#endif

#if UE_BUILD_DEBUG
	SIZE_T OldSize = 0;
	if (Ptr)
	{
		OldSize = scalable_msize(Ptr);
		if (NewSize < OldSize)
		{
			FMemory::Memset((uint8*)Ptr + NewSize, DEBUG_FILL_FREED, OldSize - NewSize); 
		}
	}
#endif
	void* NewPtr = nullptr;
#if PLATFORM_MAC
	// macOS expects all allocations to be aligned to 16 bytes, but TBBs default alignment is 8, so on Mac we always have to use scalable_aligned_realloc
	Alignment = FMath::Max((uint32)16, Alignment);
	NewPtr = scalable_aligned_realloc(Ptr, NewSize, Alignment);
#else
	if (Alignment != DEFAULT_ALIGNMENT)
	{
		Alignment = FMath::Max(NewSize >= 16 ? (uint32)16 : (uint32)8, Alignment);
		NewPtr = scalable_aligned_realloc(Ptr, NewSize, Alignment);
	}
	else
	{
		// Fulfill the promise of DEFAULT_ALIGNMENT, which aligns 16-byte or larger structures to 16 bytes,
		// while TBB aligns to 8 by default.
		NewPtr = scalable_aligned_realloc( Ptr, NewSize, NewSize >= 16 ? (uint32)16 : (uint32)8);
	}
#endif
#if UE_BUILD_DEBUG
	if (NewPtr && NewSize > OldSize )
	{
		FMemory::Memset((uint8*)NewPtr + OldSize, DEBUG_FILL_NEW, scalable_msize(NewPtr) -OldSize);
	}
#endif

	return NewPtr;
}

void* FMallocTBB::Realloc(void* Ptr, SIZE_T NewSize, uint32 Alignment)
{
	void* Result = TryRealloc(Ptr, NewSize, Alignment);

	if (Result == nullptr && NewSize)
	{
		OutOfMemory(NewSize, Alignment);
	}

	return Result;
}

void FMallocTBB::Free( void* Ptr )
{
	if( !Ptr )
	{
		return;
	}
#if UE_BUILD_DEBUG
	FMemory::Memset(Ptr, DEBUG_FILL_FREED, scalable_msize(Ptr)); 
#endif
	scalable_free(Ptr);

}

bool FMallocTBB::GetAllocationSize(void *Original, SIZE_T &SizeOut)
{
	SizeOut = scalable_msize(Original);
	return true;
}

void FMallocTBB::Trim(bool bTrimThreadCaches)
{
// TBB memory trimming might impact performance so it is only enabled in editor for now where large thread pools are used
// and more likely to do allocation migration between threads.
#if WITH_EDITOR
	scalable_allocation_command(bTrimThreadCaches ? TBBMALLOC_CLEAN_ALL_BUFFERS : TBBMALLOC_CLEAN_THREAD_BUFFERS, 0);
#endif
}

#undef DEBUG_FILL_FREED
#undef DEBUG_FILL_NEW

#endif // TBBMALLOC_ENABLED
