// Copyright Epic Games, Inc. All Rights Reserved.

#include "HAL/MallocTimer.h"

#if UE_TIME_VIRTUALMALLOC
#include "ProfilingDebugging/CsvProfiler.h"
#if CSV_PROFILER
CSV_DEFINE_CATEGORY_MODULE(CORE_API, VirtualMemory, true);
#endif
uint64 FScopedVirtualMallocTimer::GTotalCycles[FScopedVirtualMallocTimer::IndexType::Max][PlatormIndexType::PlatormIndexTypeMax] = { 0 };
uint64 FScopedVirtualMallocTimer::GTotalCounts[FScopedVirtualMallocTimer::IndexType::Max][PlatormIndexType::PlatormIndexTypeMax] = { 0 };

void FScopedVirtualMallocTimer::UpdateStats()
{
	static uint64 GLastTotalCycles[IndexType::Max][PlatormIndexType::PlatormIndexTypeMax] = { 0 };
	static uint64 GLastFrame = 0;

	uint64 Frames = GFrameCounter - GLastFrame;
	if (Frames)
	{
		GLastFrame = GFrameCounter;
		// not atomic; we assume the error is minor
		uint64 TotalCycles[IndexType::Max][PlatormIndexType::PlatormIndexTypeMax] = { 0 };
		float TotalSeconds = 0.0f;
		for (int32 Comp = 0; Comp < IndexType::Max; Comp++)
		{
			for (int32 PlatformType = 0; PlatformType < PlatormIndexType::PlatormIndexTypeMax; PlatformType++)
			{
				TotalCycles[Comp][PlatformType] = GTotalCycles[Comp][PlatformType] - GLastTotalCycles[Comp][PlatformType];
				//TotalCycles[Comp][PlatformType] = GTotalCycles[Comp][PlatformType];
				GLastTotalCycles[Comp][PlatformType] = GTotalCycles[Comp][PlatformType];
				TotalSeconds += 1000.0f * float(FPlatformTime::GetSecondsPerCycle64()) * float(TotalCycles[Comp][PlatformType]);
			}
		}
#if 1	// extra detail
		CSV_CUSTOM_STAT(VirtualMemory, Reserve_OrdinaryCPU, 1000.0f * float(FPlatformTime::GetSecondsPerCycle64()) * float(TotalCycles[0][0]), ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT(VirtualMemory, Reserve_GPU_WriteCombine, 1000.0f * float(FPlatformTime::GetSecondsPerCycle64()) * float(TotalCycles[0][1]), ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT(VirtualMemory, Reserve_GPU_Cacheable, 1000.0f * float(FPlatformTime::GetSecondsPerCycle64()) * float(TotalCycles[0][2]), ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT(VirtualMemory, Reserve_GPU_WriteCombineRenderTarget, 1000.0f * float(FPlatformTime::GetSecondsPerCycle64()) * float(TotalCycles[0][3]), ECsvCustomStatOp::Set);

		CSV_CUSTOM_STAT(VirtualMemory, Commit_OrdinaryCPU, 1000.0f * float(FPlatformTime::GetSecondsPerCycle64()) * float(TotalCycles[1][0]), ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT(VirtualMemory, Commit_GPU_WriteCombine, 1000.0f * float(FPlatformTime::GetSecondsPerCycle64()) * float(TotalCycles[1][1]), ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT(VirtualMemory, Commit_GPU_Cacheable, 1000.0f * float(FPlatformTime::GetSecondsPerCycle64()) * float(TotalCycles[1][2]), ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT(VirtualMemory, Commit_GPU_WriteCombineRenderTarget, 1000.0f * float(FPlatformTime::GetSecondsPerCycle64()) * float(TotalCycles[1][3]), ECsvCustomStatOp::Set);

		CSV_CUSTOM_STAT(VirtualMemory, Combined_OrdinaryCPU, 1000.0f * float(FPlatformTime::GetSecondsPerCycle64()) * float(TotalCycles[2][0]), ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT(VirtualMemory, Combined_GPU_WriteCombine, 1000.0f * float(FPlatformTime::GetSecondsPerCycle64()) * float(TotalCycles[2][1]), ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT(VirtualMemory, Combined_GPU_Cacheable, 1000.0f * float(FPlatformTime::GetSecondsPerCycle64()) * float(TotalCycles[2][2]), ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT(VirtualMemory, Combined_GPU_WriteCombineRenderTarget, 1000.0f * float(FPlatformTime::GetSecondsPerCycle64()) * float(TotalCycles[2][3]), ECsvCustomStatOp::Set);

		CSV_CUSTOM_STAT(VirtualMemory, DeCommit_OrdinaryCPU, 1000.0f * float(FPlatformTime::GetSecondsPerCycle64()) * float(TotalCycles[3][0]), ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT(VirtualMemory, DeCommit_GPU_WriteCombine, 1000.0f * float(FPlatformTime::GetSecondsPerCycle64()) * float(TotalCycles[3][1]), ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT(VirtualMemory, DeCommit_GPU_Cacheable, 1000.0f * float(FPlatformTime::GetSecondsPerCycle64()) * float(TotalCycles[3][2]), ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT(VirtualMemory, DeCommit_GPU_WriteCombineRenderTarget, 1000.0f * float(FPlatformTime::GetSecondsPerCycle64()) * float(TotalCycles[3][3]), ECsvCustomStatOp::Set);

		CSV_CUSTOM_STAT(VirtualMemory, Free_OrdinaryCPU, 1000.0f * float(FPlatformTime::GetSecondsPerCycle64()) * float(TotalCycles[4][0]), ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT(VirtualMemory, Free_GPU_WriteCombine, 1000.0f * float(FPlatformTime::GetSecondsPerCycle64()) * float(TotalCycles[4][1]), ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT(VirtualMemory, Free_GPU_Cacheable, 1000.0f * float(FPlatformTime::GetSecondsPerCycle64()) * float(TotalCycles[4][2]), ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT(VirtualMemory, Free_GPU_WriteCombineRenderTarget, 1000.0f * float(FPlatformTime::GetSecondsPerCycle64()) * float(TotalCycles[4][3]), ECsvCustomStatOp::Set);


		CSV_CUSTOM_STAT(VirtualMemory, ReserveCount_OrdinaryCPU, float(GTotalCounts[0][0]), ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT(VirtualMemory, ReserveCount_GPU_WriteCombine, float(GTotalCounts[0][1]), ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT(VirtualMemory, ReserveCount_GPU_Cacheable, float(GTotalCounts[0][2]), ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT(VirtualMemory, ReserveCount_GPU_WriteCombineRenderTarget, float(GTotalCounts[0][3]), ECsvCustomStatOp::Set);

		CSV_CUSTOM_STAT(VirtualMemory, CommitCount_OrdinaryCPU, float(GTotalCounts[1][0]), ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT(VirtualMemory, CommitCount_GPU_WriteCombine, float(GTotalCounts[1][1]), ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT(VirtualMemory, CommitCount_GPU_Cacheable, float(GTotalCounts[1][2]), ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT(VirtualMemory, CommitCount_GPU_WriteCombineRenderTarget, float(GTotalCounts[1][3]), ECsvCustomStatOp::Set);

		CSV_CUSTOM_STAT(VirtualMemory, CombinedCount_OrdinaryCPU, float(GTotalCounts[2][0]), ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT(VirtualMemory, CombinedCount_GPU_WriteCombine, float(GTotalCounts[2][1]), ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT(VirtualMemory, CombinedCount_GPU_Cacheable, float(GTotalCounts[2][2]), ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT(VirtualMemory, CombinedCount_GPU_WriteCombineRenderTarget, float(GTotalCounts[2][3]), ECsvCustomStatOp::Set);

		CSV_CUSTOM_STAT(VirtualMemory, DeCommitCount_OrdinaryCPU, float(GTotalCounts[3][0]), ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT(VirtualMemory, DeCommitCount_GPU_WriteCombine, float(GTotalCounts[3][1]), ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT(VirtualMemory, DeCommitCount_GPU_Cacheable, float(GTotalCounts[3][2]), ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT(VirtualMemory, DeCommitCount_GPU_WriteCombineRenderTarget, float(GTotalCounts[3][3]), ECsvCustomStatOp::Set);

		CSV_CUSTOM_STAT(VirtualMemory, FreeCount_OrdinaryCPU, float(GTotalCounts[4][0]), ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT(VirtualMemory, FreeCount_GPU_WriteCombine, float(GTotalCounts[4][1]), ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT(VirtualMemory, FreeCount_GPU_Cacheable, float(GTotalCounts[4][2]), ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT(VirtualMemory, FreeCount_GPU_WriteCombineRenderTarget, float(GTotalCounts[4][3]), ECsvCustomStatOp::Set);
#endif
		CSV_CUSTOM_STAT(VirtualMemory, TotalInSeconds, TotalSeconds, ECsvCustomStatOp::Set);
	}
}
#endif
