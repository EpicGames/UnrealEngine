// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MallocVerify.cpp: Helper class to track memory allocations
=============================================================================*/

#include "HAL/MallocVerify.h"
#include "Logging/LogMacros.h"

#if MALLOC_VERIFY

void FMallocVerify::Malloc(void* Ptr)
{
	if (Ptr)
	{
		UE_CLOGF(AllocatedPointers.Contains(Ptr), LogMemory, Fatal, "Malloc allocated pointer that's already been allocated: 0x%016llx", (int64)(PTRINT)Ptr);

		AllocatedPointers.Add(Ptr);			
	}
}

void FMallocVerify::Realloc(void* OldPtr, void* NewPtr)
{
	if (OldPtr != NewPtr)
	{			
		if (OldPtr)
		{
			UE_CLOGF(!AllocatedPointers.Contains(OldPtr), LogMemory, Fatal, "Realloc entered with old pointer that hasn't been allocated yet: 0x%016llx", (int64)(PTRINT)OldPtr);

			AllocatedPointers.Remove(OldPtr);
		}
		if (NewPtr)
		{
			UE_CLOGF(AllocatedPointers.Contains(NewPtr), LogMemory, Fatal, "Realloc allocated new pointer that has already been allocated: 0x%016llx", (int64)(PTRINT)NewPtr);

			AllocatedPointers.Add(NewPtr);
		}
	}
	else if (OldPtr)
	{
		UE_CLOGF(!AllocatedPointers.Contains(OldPtr), LogMemory, Fatal, "Realloc entered with old pointer that hasn't been allocated yet: 0x%016llx", (int64)(PTRINT)OldPtr);
	}
}

void FMallocVerify::Free(void* Ptr)
{
	if (Ptr)
	{
		UE_CLOGF(!AllocatedPointers.Contains(Ptr), LogMemory, Fatal, "Free attempts to free a pointer that hasn't been allocated yet: 0x%016llx", (int64)(PTRINT)Ptr);

		AllocatedPointers.Remove(Ptr);
	}
}

#endif // MALLOC_VERIFY
