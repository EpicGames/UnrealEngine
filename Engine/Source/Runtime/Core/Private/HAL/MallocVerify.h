// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MallocVerify.h: Helper class to track memory allocations
=============================================================================*/

#pragma once

#include "CoreTypes.h"
#include "HAL/MemoryBase.h"
#include "Containers/Set.h"
#include "Misc/ScopeLock.h"
#include "HAL/Allocators/AnsiAllocator.h"

#define MALLOC_VERIFY 0

#if MALLOC_VERIFY

/**
 * Maintains a list of all pointers to currently allocated memory.
 */
class FMallocVerify
{
	/** List of all currently allocated pointers */
	TSet<void*, DefaultKeyFuncs<void*>, FAnsiSetAllocator> AllocatedPointers;

public:

	/** Handles new allocated pointer */
	void Malloc(void* Ptr);

	/** Handles reallocation */
	void Realloc(void* OldPtr, void* NewPtr);

	/** Removes allocated pointer from list */
	void Free(void* Ptr);
};

/**
 * A verifying proxy malloc that takes a malloc to be used and checks that the caller
 * is passing valid pointers.
 */
class FMallocVerifyProxy : public FMalloc
{
private:
	/** Malloc we're based on, aka using under the hood */
	FMalloc* UsedMalloc;

	/* Verifier object */
	FMallocVerify Verify;

	/** Malloc critical section */
	FCriticalSection VerifyCritical;

public:
	explicit FMallocVerifyProxy(FMalloc* InMalloc)
		: UsedMalloc(InMalloc)
	{
	}

	virtual void* Malloc(SIZE_T Size, uint32 Alignment) override
	{
		FScopeLock VerifyLock(&VerifyCritical);
		void* Result = UsedMalloc->Malloc(Size, Alignment);
		Verify.Malloc(Result);
		return Result;
	}

	virtual void* Realloc(void* Ptr, SIZE_T NewSize, uint32 Alignment) override
	{
		FScopeLock VerifyLock(&VerifyCritical);
		void* Result = UsedMalloc->Realloc(Ptr, NewSize, Alignment);
		Verify.Realloc(Ptr, Result);
		return Result;
	}

	virtual void Free(void* Ptr) override
	{
		if (Ptr)
		{
			FScopeLock VerifyLock(&VerifyCritical);
			Verify.Free(Ptr);
			UsedMalloc->Free(Ptr);
		}
	}

	virtual void InitializeStatsMetadata() override
	{
		UsedMalloc->InitializeStatsMetadata();
	}

	virtual void GetAllocatorStats(FGenericMemoryStats& OutStats) override
	{
		UsedMalloc->GetAllocatorStats(OutStats);
	}

	virtual void DumpAllocatorStats(FOutputDevice& Ar) override
	{
		UsedMalloc->DumpAllocatorStats(Ar);
	}

	virtual bool ValidateHeap() override
	{
		return UsedMalloc->ValidateHeap();
	}

	virtual bool Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override
	{
		return UsedMalloc->Exec(InWorld, Cmd, Ar);
	}

	virtual bool GetAllocationSize(void* Original, SIZE_T& OutSize) override
	{
		return UsedMalloc->GetAllocationSize(Original, OutSize);
	}
	virtual SIZE_T QuantizeSize(SIZE_T Count, uint32 Alignment) override
	{
		return UsedMalloc->QuantizeSize(Count, Alignment);
	}
	virtual void Trim(bool bTrimThreadCaches) override
	{
		return UsedMalloc->Trim(bTrimThreadCaches);
	}
	virtual void SetupTLSCachesOnCurrentThread() override
	{
		return UsedMalloc->SetupTLSCachesOnCurrentThread();
	}
	virtual void MarkTLSCachesAsUsedOnCurrentThread() override
	{
		UsedMalloc->MarkTLSCachesAsUsedOnCurrentThread();
	}

	virtual void MarkTLSCachesAsUnusedOnCurrentThread() override
	{
		UsedMalloc->MarkTLSCachesAsUnusedOnCurrentThread();
	}
	virtual void ClearAndDisableTLSCachesOnCurrentThread() override
	{
		return UsedMalloc->ClearAndDisableTLSCachesOnCurrentThread();
	}

	virtual const TCHAR* GetDescriptiveName() override
	{ 
		return UsedMalloc->GetDescriptiveName();
	}
};

#endif // MALLOC_VERIFY
