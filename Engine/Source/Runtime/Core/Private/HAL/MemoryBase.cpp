// Copyright Epic Games, Inc. All Rights Reserved.

#include "HAL/MemoryBase.h"
#include "Stats/Stats.h"
#include "HAL/MemoryMisc.h"

/** Memory allocator base stats. */
DECLARE_DWORD_COUNTER_STAT(TEXT("Malloc calls"),			STAT_MallocCalls,STATGROUP_MemoryAllocator);
DECLARE_DWORD_COUNTER_STAT(TEXT("Free calls"),				STAT_FreeCalls,STATGROUP_MemoryAllocator);
DECLARE_DWORD_COUNTER_STAT(TEXT("Realloc calls"),			STAT_ReallocCalls,STATGROUP_MemoryAllocator);
DECLARE_DWORD_COUNTER_STAT(TEXT("Total Allocator calls"),	STAT_TotalAllocatorCalls,STATGROUP_MemoryAllocator);

#if !UE_BUILD_SHIPPING
TAtomic<uint64> FMalloc::MaxSingleAlloc;
#endif

struct FCurrentFrameCalls
{
	uint32 LastMallocCalls;
	uint32 LastReallocCalls;
	uint32 LastFreeCalls;
	
	uint32 MallocCalls;
	uint32 ReallocCalls;
	uint32 FreeCalls;
	uint32 AllocatorCalls;

	FCurrentFrameCalls()
		: LastMallocCalls(0)
		, LastReallocCalls(0)
		, LastFreeCalls(0)
		, MallocCalls(0)
		, ReallocCalls(0)
		, FreeCalls(0)
		, AllocatorCalls(0)
	{}

	void Update()
	{
		uint32 LocalTotalMallocCalls  = 0;
		uint32 LocalTotalReallocCalls = 0;
		uint32 LocalTotalFreeCalls    = 0;

		MallocCalls      = LocalTotalMallocCalls - LastMallocCalls;
		ReallocCalls     = LocalTotalReallocCalls - LastReallocCalls;
		FreeCalls        = LocalTotalFreeCalls - LastFreeCalls;
		AllocatorCalls   = MallocCalls + ReallocCalls + FreeCalls;

		LastMallocCalls  = LocalTotalMallocCalls;
		LastReallocCalls = LocalTotalReallocCalls;
		LastFreeCalls    = LocalTotalFreeCalls;
	}
};

static FCurrentFrameCalls& GetCurrentFrameCalls()
{
	static FCurrentFrameCalls CurrentFrameCalls;
	return CurrentFrameCalls;
}

void FMalloc::InitializeStatsMetadata()
{
	// Initialize stats metadata here instead of UpdateStats.
	// Mostly to avoid dead-lock when stats malloc profiler is enabled.
	GET_STATFNAME(STAT_MallocCalls);
	GET_STATFNAME(STAT_ReallocCalls);
	GET_STATFNAME(STAT_FreeCalls);
	GET_STATFNAME(STAT_TotalAllocatorCalls);
}


void FMalloc::UpdateStats()
{
#if	STATS
	GetCurrentFrameCalls().Update();

	SET_DWORD_STAT( STAT_MallocCalls, GetCurrentFrameCalls().MallocCalls );
	SET_DWORD_STAT( STAT_ReallocCalls, GetCurrentFrameCalls().ReallocCalls );
	SET_DWORD_STAT( STAT_FreeCalls, GetCurrentFrameCalls().FreeCalls );
	SET_DWORD_STAT( STAT_TotalAllocatorCalls, GetCurrentFrameCalls().AllocatorCalls );
#endif
}


void FMalloc::GetAllocatorStats( FGenericMemoryStats& out_Stats )
{
#if	STATS
	out_Stats.Add( TEXT("Malloc calls"), GetCurrentFrameCalls().MallocCalls );
	out_Stats.Add( TEXT("Realloc calls"), GetCurrentFrameCalls().ReallocCalls );
	out_Stats.Add( TEXT("Free calls"), GetCurrentFrameCalls().FreeCalls );
	out_Stats.Add( TEXT("Total Allocator calls"), GetCurrentFrameCalls().AllocatorCalls );
#endif // STATS
}

void* FMalloc::TryMalloc(SIZE_T Count, uint32 Alignment)
{
	return Malloc(Count, Alignment);
}

void* FMalloc::TryRealloc(void* Original, SIZE_T Count, uint32 Alignment)
{
	return Realloc(Original, Count, Alignment);
}

void* FMalloc::MallocZeroed(SIZE_T Count, uint32 Alignment)
{
	void* const Memory = Malloc(Count, Alignment);

	if (Memory)
	{
		FPlatformMemory::Memzero(Memory, Count);
	}

	return Memory;
}

void* FMalloc::TryMallocZeroed(SIZE_T Count, uint32 Alignment)
{
	void* const Memory = Malloc(Count, Alignment);

	if (Memory)
	{
		FPlatformMemory::Memzero(Memory, Count);
	}

	return Memory;
}
