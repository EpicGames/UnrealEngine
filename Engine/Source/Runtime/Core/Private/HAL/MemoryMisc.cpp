// Copyright Epic Games, Inc. All Rights Reserved.

#include "HAL/MemoryMisc.h"
#include "CoreGlobals.h"


namespace MemoryMiscInternal
{
	static constexpr float MemDeltaInMB(uint64 Current, uint64 Previous)
	{
		return ((static_cast<float>(Current) - static_cast<float>(Previous)) / (1024.f * 1024.f));
	}

	static constexpr float BytesToMB(uint64 Bytes)
	{
		return (static_cast<float>(Bytes) / (1024.0f * 1024.0f));
	}

	void LogMemoryDiff(const TCHAR* Context, const FPlatformMemoryStats& StartStats, const FPlatformMemoryStats& EndStats)
	{
		UE_LOGF(LogMemory, Log, "ScopedMemoryStat[%ls] UsedPhysical %.02fMB (%+.02fMB), PeakPhysical: %.02fMB (%+.02fMB), UsedVirtual: %.02fMB (%+.02fMB) PeakVirtual: %.02fMB (%+.02fMB)",
			Context,
			BytesToMB(EndStats.UsedPhysical),
			MemDeltaInMB(EndStats.UsedPhysical, StartStats.UsedPhysical),
			BytesToMB(EndStats.PeakUsedPhysical),
			MemDeltaInMB(EndStats.PeakUsedPhysical, StartStats.PeakUsedPhysical),
			BytesToMB(EndStats.UsedVirtual),
			MemDeltaInMB(EndStats.UsedVirtual, StartStats.UsedVirtual),
			BytesToMB(EndStats.PeakUsedVirtual),
			MemDeltaInMB(EndStats.PeakUsedVirtual, StartStats.PeakUsedVirtual));
	}

#if PLATFORM_UNIX
	void LogSharedMemoryDiff(const TCHAR* Context, const FExtendedPlatformMemoryStats& StartStats, const FExtendedPlatformMemoryStats& EndStats)
	{
		if (EndStats.bIs_KSM_Enabled)
		{
			UE_LOGF(LogMemory, Log, "SharedMemoryTracker[%ls] Unique: %.02fMB (%+.02fMB), Shared: %.02fMB (%+.02fMB), KSM Profit: %.02fMB (%+.02fMB), Shared To Unique: %.02fMB (%+.02fMB)",
				   Context,
				   BytesToMB(EndStats.Private_Clean + EndStats.Private_Dirty),
				   MemDeltaInMB(EndStats.Private_Clean + EndStats.Private_Dirty, StartStats.Private_Clean + StartStats.Private_Dirty),
				   BytesToMB(EndStats.Shared_Clean + EndStats.Shared_Dirty),
				   MemDeltaInMB(EndStats.Shared_Clean + EndStats.Shared_Dirty, StartStats.Shared_Clean + StartStats.Shared_Dirty),
				   BytesToMB(EndStats.KSM_Process_Profit),
				   MemDeltaInMB(EndStats.KSM_Process_Profit, StartStats.KSM_Process_Profit),
				   BytesToMB(EndStats.Shared_Memory_COW_Bytes),
				   MemDeltaInMB(EndStats.Shared_Memory_COW_Bytes, StartStats.Shared_Memory_COW_Bytes));
		}
		else
		{
			UE_LOGF(LogMemory, Log, "SharedMemoryTracker[%ls] Unique: %.02fMB (%+.02fMB), Shared: %.02fMB (%+.02fMB), Shared To Unique: %.02fMB (%+.02fMB)",
				   Context,
				   BytesToMB(EndStats.Private_Clean + EndStats.Private_Dirty),
				   MemDeltaInMB(EndStats.Private_Clean + EndStats.Private_Dirty, StartStats.Private_Clean + StartStats.Private_Dirty),
				   BytesToMB(EndStats.Shared_Clean + EndStats.Shared_Dirty),
				   MemDeltaInMB(EndStats.Shared_Clean + EndStats.Shared_Dirty, StartStats.Shared_Clean + StartStats.Shared_Dirty),
				   BytesToMB(EndStats.Shared_Memory_COW_Bytes),
				   MemDeltaInMB(EndStats.Shared_Memory_COW_Bytes, StartStats.Shared_Memory_COW_Bytes));
		}
	}
#endif //PLATFORM_UNIX
}

#if ENABLE_MEMORY_SCOPE_STATS
FScopedMemoryStats::FScopedMemoryStats(const TCHAR* Name)
	: Text(Name)
	, StartStats(FPlatformMemory::GetStats())
{
}

FScopedMemoryStats::~FScopedMemoryStats()
{
	const FPlatformMemoryStats EndStats = FPlatformMemory::GetStats();
	MemoryMiscInternal::LogMemoryDiff(Text, StartStats, EndStats);
}
#endif

#if ENABLE_SHARED_MEMORY_TRACKER && PLATFORM_UNIX

void FSharedMemoryTracker::PrintMemoryDiff(const TCHAR* Context)
{
	static FExtendedPlatformMemoryStats LastStats = { 0,0,0,0 };
	const FExtendedPlatformMemoryStats CurrentStats = FPlatformMemory::GetExtendedStats();

	MemoryMiscInternal::LogSharedMemoryDiff(Context, LastStats, CurrentStats);

	LastStats = CurrentStats;
}

FSharedMemoryTracker::FSharedMemoryTracker(const FString& InContext)
	: PrintContext(InContext)
	, StartStats(FPlatformMemory::GetExtendedStats())
{

}

FSharedMemoryTracker::~FSharedMemoryTracker()
{
	const FExtendedPlatformMemoryStats EndStats = FPlatformMemory::GetExtendedStats();
	MemoryMiscInternal::LogSharedMemoryDiff(*PrintContext, StartStats, EndStats);
}

#endif
