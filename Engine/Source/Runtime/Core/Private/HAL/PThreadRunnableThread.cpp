// Copyright Epic Games, Inc. All Rights Reserved.

#include "HAL/PThreadRunnableThread.h"
#include "Stats/Stats.h"

#if	PLATFORM_USE_PTHREADS


uint32 FRunnableThreadPThread::Run()
{
	// Assume we'll fail init
	uint32 ExitCode = 1;
	check(Runnable);

	// Initialize the runnable object
	if (Runnable->Init() == true)
	{
		// Initialization has completed, release the sync event
		ThreadInitSyncEvent->Trigger();

		// Setup TLS for this thread, used by FTlsAutoCleanup objects.
		SetTls();

		// Now run the task that needs to be done
		ExitCode = Runnable->Run();
		// Allow any allocated resources to be cleaned up
		Runnable->Exit();

#if STATS
		FThreadStats::Shutdown();
#endif
		FreeTls();
	}
	else
	{
		// Initialization has failed, release the sync event
		ThreadInitSyncEvent->Trigger();
	}

	return ExitCode;
}


#endif // PLATFORM_USE_PTHREADS
