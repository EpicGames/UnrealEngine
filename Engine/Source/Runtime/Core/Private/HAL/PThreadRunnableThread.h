// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_UNSUPPORTED - Unsupported platform

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "HAL/UnrealMemory.h"
#include "Containers/StringConv.h"
#include "Logging/LogMacros.h"
#include "CoreGlobals.h"
#include "HAL/Event.h"
#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "HAL/PlatformProcess.h"
#include "HAL/ThreadManager.h"

#if	PLATFORM_USE_PTHREADS

/**
 * This is the base interface for all runnable thread classes. It specifies the
 * methods used in managing its life cycle.
 */
class FRunnableThreadPThread : public FRunnableThread
{

protected:

	/**
	 * The thread handle for the thread
	 */
	pthread_t Thread;

	/**
	 * If true, the thread handle needs pthread_join()
	 */
	bool bThreadStartedAndNotCleanedUp;

	typedef void *(*PthreadEntryPoint)(void *arg);

	/**
	 * Converts an EThreadPriority to a value that can be used in pthread_setschedparam. Virtual
	 * so that platforms can override priority values
	 */
	virtual int32 TranslateThreadPriority(EThreadPriority Priority)
	{
		// these are some default priorities
		switch (Priority)
		{
			// 0 is the lowest, 31 is the highest possible priority for pthread
			case TPri_Highest: case TPri_TimeCritical: return 30;
			case TPri_AboveNormal: return 25;
			case TPri_Normal: return 15;
			case TPri_BelowNormal: return 5;
			case TPri_Lowest: return 1;
			case TPri_SlightlyBelowNormal: return 14;
			case TPri_Num:
			default:
				UE_LOGF(LogHAL, Fatal, "Unknown Priority passed to FRunnableThreadPThread::TranslateThreadPriority()");
				return 15;
		}
	}

	virtual void SetThreadPriority(pthread_t InThread, EThreadPriority NewPriority)
	{
		struct sched_param Sched;
		FMemory::Memzero(&Sched, sizeof(struct sched_param));
		int32 Policy = SCHED_RR;

		// Read the current policy
		pthread_getschedparam(InThread, &Policy, &Sched);

		// set the priority appropriately
		Sched.sched_priority = TranslateThreadPriority(NewPriority);
		pthread_setschedparam(InThread, Policy, &Sched);
	}

	/**
	 * Wrapper for pthread_create that takes a name
	 *
	 * Allows a subclass to override this function to create a thread and give it a name, if
	 * the platform supports it
	 *
	 * Takes the same arguments as pthread_create
	 */
	virtual int CreateThreadWithName(pthread_t* HandlePtr, pthread_attr_t* AttrPtr, PthreadEntryPoint Proc, void* Arg, const ANSICHAR* /*Name*/)
	{
		// by default, we ignore the name since it's not in the standard pthreads
		return pthread_create(HandlePtr, AttrPtr, Proc, Arg);
	}

	/**
	 * Allows platforms to choose a default stack size for when a StackSize of 0 is given
	 */
	virtual int GetDefaultStackSize()
	{
		// Some information on default stack sizes, selected when given 0:
		// - On Windows, all threads get 1MB: http://msdn.microsoft.com/en-us/library/windows/desktop/ms686774(v=vs.85).aspx
		// - On Mac, main thread gets 8MB,
		// - all other threads get 512 kB when created through pthread or NSThread, and only 4kB when through MPTask()
		//	( http://developer.apple.com/library/mac/#qa/qa1419/_index.html )
		return 0;
	}

	/**
	 * Allows platforms to adjust stack size
	 */
	virtual uint32 AdjustStackSize(uint32 InStackSize)
	{
		// allow the platform to override default stack size
		if (InStackSize == 0)
		{
			InStackSize = GetDefaultStackSize();
		}
		return InStackSize;
	}

	virtual bool SetThreadAttr(pthread_attr_t *ThreadAttr, uint32 InStackSize, EThreadCreateFlags InCreateFlags)
	{
		// allow platform to adjust stack size
		InStackSize = AdjustStackSize(InStackSize);

		if (InStackSize != 0)
		{
			if (pthread_attr_init(ThreadAttr) == 0)
			{
				// we'll use this the attribute if this succeeds, otherwise, we'll wing it without it.
				const size_t StackSize = (size_t)InStackSize;
				if (pthread_attr_setstacksize(ThreadAttr, StackSize) != 0)
				{
					UE_LOGF(LogHAL, Log, "Failed to change pthread stack size to %d bytes", (int)InStackSize);
				}
				return true;
			}
			else
			{
				UE_LOGF(LogHAL, Log, "Failed to init thread attr");
			}
		}

		return false;
	}

	bool SpinPthread(pthread_t* HandlePtr, bool* OutThreadCreated, PthreadEntryPoint Proc, uint32 InStackSize, EThreadCreateFlags InCreateFlags, void *Arg)
	{
		*OutThreadCreated = false;
		pthread_attr_t ThreadAttr;
		pthread_attr_t *ThreadAttrPtr = SetThreadAttr(&ThreadAttr, InStackSize, InCreateFlags) ? &ThreadAttr : nullptr;

		const int ThreadErrno = CreateThreadWithName(HandlePtr, ThreadAttrPtr, Proc, Arg, TCHAR_TO_ANSI(*ThreadName));
		*OutThreadCreated = (ThreadErrno == 0);
		if (ThreadAttrPtr)
		{
			pthread_attr_destroy(ThreadAttrPtr);
		}

		// Move the thread to the specified processors if requested
		if (!*OutThreadCreated)
		{
			UE_LOGF(LogHAL, Log, "Failed to create thread! (err=%d, %ls)", ThreadErrno, UTF8_TO_TCHAR(strerror(ThreadErrno)));
		}

		return *OutThreadCreated;
	}

	/**
	 * The thread entry point. Simply forwards the call on to the right
	 * thread main function
	 */
	static void *STDCALL _ThreadProc(void *pThis)
	{
		check(pThis);

		FRunnableThreadPThread* ThisThread = (FRunnableThreadPThread*)pThis;

		// cache the thread ID for this thread (defined by the platform)
		ThisThread->ThreadID = FPlatformTLS::GetCurrentThreadId();

		if (ThisThread->ThreadID == ~uint32(0)) [[unlikely]]
		{
			LowLevelFatalError(TEXT("Thread ID is -1. Reconsider implementation of FThread::InvalidThreadId"));
		}

		FThreadManager::Get().AddThread(ThisThread->ThreadID, ThisThread);

		// set the affinity.  This function sets affinity on the current thread, so don't call in the Create function which will trash the main thread affinity.
		FPlatformProcess::SetThreadAffinityMask(ThisThread->ThreadAffinityMask);		

		// run the thread!
		ThisThread->PreRun();
		ThisThread->Run();
		ThisThread->PostRun();

		pthread_exit(NULL);
		return NULL;
	}

	virtual PthreadEntryPoint GetThreadEntryPoint()
	{
		return _ThreadProc;
	}

	/**
	 * Allows a platform subclass to setup anything needed on the thread before running the Run function
	 */
	virtual void PreRun()
	{
	}

	/**
	 * Allows a platform subclass to teardown anything needed on the thread after running the Run function
	 */
	virtual void PostRun()
	{
	}

	/**
	 * The real thread entry point. It calls the Init/Run/Exit methods on
	 * the runnable object
	 */
	uint32 Run();

	/**
	 * Internal helper, needs to be called by subclasses that override virtual functions.
	 * Should be idempotent since it will be called multiple times.
	 *
	 * Why it is needed:
	 *  if the thread is still running, Kill() will Stop() it, resulting in _ThreadProc() calling PostRun(). However,
	 * PostRun() is a virtual function and if the FRunnable subclass overrides it, wrong one will be called if we are doing that
	 * in the destructor of the base class.
	 */
	void FRunnableThreadPThread_DestructorBody()
	{
		// Clean up our thread if it is still active
		if (bThreadStartedAndNotCleanedUp)
		{
			Kill(true);
		}
		checkf(!bThreadStartedAndNotCleanedUp, TEXT("Thread still not cleaned up after Kill(true)!"));
		ThreadID = 0;
	}

public:
	FRunnableThreadPThread()
		: bThreadStartedAndNotCleanedUp(false)
	{
	}

	~FRunnableThreadPThread()
	{
		FRunnableThreadPThread_DestructorBody();
	}

	virtual void SetThreadPriority(EThreadPriority NewPriority) override
	{
		// Don't bother calling the OS if there is no need
		if (NewPriority != ThreadPriority)
		{
			ThreadPriority = NewPriority;
			SetThreadPriority(Thread, NewPriority);
		}
	}

	virtual void Suspend(bool bShouldPause = 1) override
	{
		check(Thread);
		// Impossible in pthreads!
	}

	virtual bool Kill(bool bShouldWait) override
	{
		check(Thread && "Did you forget to call Create()?");
		bool bDidExitOK = true;
		// Let the runnable have a chance to stop without brute force killing
		if (Runnable)
		{
			Runnable->Stop();
		}
		// If waiting was specified, wait the amount of time. If that fails,
		// brute force kill that thread. Very bad as that might leak.
		if (bShouldWait && bThreadStartedAndNotCleanedUp)
		{
			int JoinResult = pthread_join(Thread, nullptr);

			UE_CLOGF(JoinResult != 0, LogHAL, Warning, "Failed to join thread %u (%ls). (err=%d, %ls)", ThreadID, *ThreadName, JoinResult, UTF8_TO_TCHAR(strerror(JoinResult)));

			bDidExitOK = (JoinResult == 0);
			bThreadStartedAndNotCleanedUp = false;
		}

		// It's not really safe to kill a pthread. So don't do it.
		return bDidExitOK;
	}

	virtual void WaitForCompletion() override
	{
		// Block until this thread exits
		if (bThreadStartedAndNotCleanedUp)
		{
			int JoinResult = pthread_join(Thread, nullptr);

			UE_CLOGF(JoinResult != 0, LogHAL, Warning, "Failed to join thread %u (%ls). (err=%d, %ls)", ThreadID, *ThreadName, JoinResult, UTF8_TO_TCHAR(strerror(JoinResult)));

			bThreadStartedAndNotCleanedUp = false;
		}
	}

protected:

	virtual bool CreateInternal(FRunnable* InRunnable, const TCHAR* InThreadName,
		uint32 InStackSize = 0,
		EThreadPriority InThreadPri = TPri_Normal, uint64 InThreadAffinityMask = 0,
		EThreadCreateFlags InCreateFlags = EThreadCreateFlags::None) override
	{
		check(InRunnable);
		Runnable = InRunnable;

		// Create a sync event to guarantee the Init() function is called first
		ThreadInitSyncEvent	= FPlatformProcess::GetSynchEventFromPool(true);
		// A name for the thread in for debug purposes. _ThreadProc will set it.
		ThreadName = InThreadName ? InThreadName : TEXT("Unnamed UE4");
		ThreadPriority = InThreadPri; // Make sure the thread priority is correct during init
		ThreadAffinityMask = InThreadAffinityMask;

		// Create the new thread
		const bool ThreadCreated = SpinPthread(&Thread, &bThreadStartedAndNotCleanedUp, GetThreadEntryPoint(), InStackSize, InCreateFlags, this);
		// If it fails, clear all the vars
		if (ThreadCreated)
		{
			// Let the thread start up, then set the name for debug purposes.
			ThreadInitSyncEvent->Wait((uint32)-1); // infinite wait

			// set the priority
			ThreadPriority = TPri_Num; // set to Num (invalid) to ensure that the priority will be set below
			SetThreadPriority(InThreadPri);
		}
		else // If it fails, clear all the vars
		{
			Runnable = NULL;
		}

		// Cleanup the sync event
		FPlatformProcess::ReturnSynchEventToPool( ThreadInitSyncEvent );
		ThreadInitSyncEvent = nullptr;
		return bThreadStartedAndNotCleanedUp;
	}
};

#endif
