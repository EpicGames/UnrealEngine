// Copyright Epic Games, Inc. All Rights Reserved.

#include "HAL/PlatformAffinity.h"

#include "Misc/Optional.h"

namespace UE::HAL
{
	/** Stores the overridden value for the audio render thread. */
	static TOptional<uint64> AudioRenderThreadMask;
}

void FPlatformAffinity::SetAudioRenderThreadMask(const uint64 InNewValue)
{
	UE::HAL::AudioRenderThreadMask.Emplace(InNewValue);
}

const uint64 FPlatformAffinity::GetAudioRenderThreadMask()
{
	return UE::HAL::AudioRenderThreadMask.Get(FPlatformAffinityBase::GetAudioRenderThreadMask());
}
