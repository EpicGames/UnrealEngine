// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GenericPlatformFile.cpp: Generic implementations of platform file I/O functions
=============================================================================*/

#include "HAL/PlatformFileManager.h"
#include "Misc/AccessDetection.h"
#include "Misc/AssertionMacros.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "Modules/ModuleManager.h"
#include "HAL/IPlatformFileLogWrapper.h"
#include "HAL/IPlatformFileCachedWrapper.h"
#include "HAL/IPlatformFileModule.h"
#include "HAL/IPlatformFileOpenLogWrapper.h"
#include "Templates/UniquePtr.h"

FPlatformFileManager::FPlatformFileManager()
	: TopmostPlatformFile(NULL)
{}

IPlatformFile& FPlatformFileManager::GetPlatformFile()
{
	if (TopmostPlatformFile == NULL)
	{
		TopmostPlatformFile = &IPlatformFile::GetPlatformPhysical();
	}
	return *TopmostPlatformFile;
}

CORE_API IPlatformFile& FPlatformFileManager::GetPlatformPhysical()
{
	if (PhysicalPlatformFile == nullptr)
	{
		PhysicalPlatformFile = &IPlatformFile::GetPlatformPhysical();
	}

	return *PhysicalPlatformFile;
}

void FPlatformFileManager::SetPlatformFile(IPlatformFile& NewTopmostPlatformFile)
{
	TopmostPlatformFile = &NewTopmostPlatformFile;
	NewTopmostPlatformFile.InitializeAfterSetActive();
}

void FPlatformFileManager::SetPlatformPhysical(IPlatformFile& NewPhysicalPlatformFile)
{
	PhysicalPlatformFile = &NewPhysicalPlatformFile;
	NewPhysicalPlatformFile.InitializeAfterSetActive();
}

IPlatformFile* FPlatformFileManager::FindPlatformFile(const TCHAR* Name)
{
	check(TopmostPlatformFile != NULL);
	for (IPlatformFile* ChainElement = TopmostPlatformFile; ChainElement; ChainElement = ChainElement->GetLowerLevel())
	{
		if (FCString::Stricmp(ChainElement->GetName(), Name) == 0)
		{
			return ChainElement;
		}
	}
	return NULL;
}

void FPlatformFileManager::TickActivePlatformFile()
{
	for ( IPlatformFile* ChainElement = TopmostPlatformFile; ChainElement; ChainElement = ChainElement->GetLowerLevel() )
	{
		ChainElement->Tick();
	}
}

IPlatformFile* FPlatformFileManager::GetPlatformFile(const TCHAR* Name)
{
	IPlatformFile* PlatformFile = NULL;

	// Check Core platform files (Profile, Log) by name.
	if (FCString::Strcmp(FLoggedPlatformFile::GetTypeName(), Name) == 0)
	{
		static TUniquePtr<IPlatformFile> AutoDestroySingleton(new FLoggedPlatformFile());
		PlatformFile = AutoDestroySingleton.Get();
	}
#if !UE_BUILD_SHIPPING
	else if (FCString::Strcmp(FPlatformFileOpenLog::GetTypeName(), Name) == 0)
	{
		static TUniquePtr<IPlatformFile> AutoDestroySingleton(new FPlatformFileOpenLog());
		PlatformFile = AutoDestroySingleton.Get();
	}
#endif
	else if (FCString::Strcmp(FCachedReadPlatformFile::GetTypeName(), Name) == 0)
	{
		static TUniquePtr<IPlatformFile> AutoDestroySingleton(new FCachedReadPlatformFile());
		PlatformFile = AutoDestroySingleton.Get();
	}
	else if (FModuleManager::Get().ModuleExists(Name))
	{
		class IPlatformFileModule* PlatformFileModule = FModuleManager::LoadModulePtr<IPlatformFileModule>(Name);

		if (PlatformFileModule != NULL)
		{
			PlatformFile = PlatformFileModule->GetPlatformFile();
		}
	}

	return PlatformFile;
}

void FPlatformFileManager::RemovePlatformFile(IPlatformFile* PlatformFileToRemove)
{
	check(TopmostPlatformFile != nullptr);
	check(PlatformFileToRemove != nullptr);

	IPlatformFile* HigherLevelPlatformFile = nullptr;
	IPlatformFile* FoundElement = nullptr;
	for (FoundElement = TopmostPlatformFile; FoundElement && FoundElement != PlatformFileToRemove; FoundElement = FoundElement->GetLowerLevel())
	{
		HigherLevelPlatformFile = FoundElement;
	}
	check(FoundElement == PlatformFileToRemove);
	if (HigherLevelPlatformFile)
	{
		check(HigherLevelPlatformFile->GetLowerLevel() == PlatformFileToRemove);
		HigherLevelPlatformFile->SetLowerLevel(PlatformFileToRemove->GetLowerLevel());
	}
	else
	{
		check(TopmostPlatformFile == PlatformFileToRemove);
		check(PlatformFileToRemove->GetLowerLevel());
		SetPlatformFile(*PlatformFileToRemove->GetLowerLevel());
	}
}

bool FPlatformFileManager::InsertPlatformFile(IPlatformFile* NewPlatformFile)
{
	check(TopmostPlatformFile != nullptr);
	check(NewPlatformFile != nullptr);

	if (FindPlatformFile(NewPlatformFile->GetName()))
	{
		return false;
	}

	if (NewPlatformFile->GetLowerLevel() == nullptr)
	{
		return false; // Physical layer must be at the bottom
	}

	if (NewPlatformFile->GetLowerLevel() == TopmostPlatformFile)
	{
		SetPlatformFile(*NewPlatformFile);
		return true;
	}

	for (IPlatformFile* ChainElement = TopmostPlatformFile; ChainElement; ChainElement = ChainElement->GetLowerLevel())
	{
		if (ChainElement->GetLowerLevel() == NewPlatformFile->GetLowerLevel())
		{
			ChainElement->SetLowerLevel(NewPlatformFile);
			return true;
		}
	}

	return false;
}

void FPlatformFileManager::InitializeNewAsyncIO()
{
	// Make sure all platform wrappers know about new async IO and EDL
	for (IPlatformFile* ChainElement = TopmostPlatformFile; ChainElement; ChainElement = ChainElement->GetLowerLevel())
	{
		ChainElement->InitializeNewAsyncIO();
	}
}

FPlatformFileManager& FPlatformFileManager::Get()
{
	UE::AccessDetection::ReportAccess(UE::AccessDetection::EType::File);
	static FPlatformFileManager Singleton;
	return Singleton;
}
