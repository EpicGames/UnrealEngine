// Copyright Epic Games, Inc. All Rights Reserved.

#include "HAL/PlatformMemory.h"

static_assert(FPlatformMemory::KernelAddressBit < 64, "Unsupported architecture, please declare which address bit distinguish user space from kernel space for non-x64/ARM64 platforms");