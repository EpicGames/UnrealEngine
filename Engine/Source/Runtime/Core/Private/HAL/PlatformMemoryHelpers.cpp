// Copyright Epic Games, Inc. All Rights Reserved.

#include "HAL/PlatformMemoryHelpers.h"

#include "CoreGlobals.h"
#include "HAL/PlatformMemory.h"

namespace PlatformMemoryHelpers
{

static uint64 CachedMemoryStatsAtFrame = -1;
static FPlatformMemoryStats CachedMemoryStats;

FPlatformMemoryStats GetFrameMemoryStats()
{
	if (IsInGameThread())
	{
		if (GFrameCounter != CachedMemoryStatsAtFrame)
		{
			CachedMemoryStatsAtFrame = GFrameCounter;
			CachedMemoryStats = FPlatformMemory::GetStats();
		}
		return CachedMemoryStats;
	}

	return FPlatformMemory::GetStats();
}

} //PlatformMemoryHelpers