// Copyright Epic Games, Inc. All Rights Reserved.

#include "HAL/Thread.h"
#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "HAL/PlatformTLS.h"
#include "HAL/ThreadSafeBool.h"
#include "HAL/LowLevelMemTracker.h"
#include "Misc/AssertionMacros.h"
#include "Templates/UnrealTemplate.h"
#include "Templates/UniquePtr.h"
#include "Misc/Fork.h"
#include "Misc/SingleThreadRunnable.h"

class FThreadImpl final : public FRunnable, public FSingleThreadRunnable
{
public:
	FThreadImpl(
		TCHAR const* ThreadName,
		TUniqueFunction<void()>&& InThreadFunction,
		TUniqueFunction<void()>&& InSingleThreadTickFunction,
		uint32 StackSize,
		EThreadPriority ThreadPriority,
		FThreadAffinity ThreadAffinity,
		FThread::EForkable IsForkable
	)
		: ThreadFunction(MoveTemp(InThreadFunction))
		, SingleThreadTickFunction(MoveTemp(InSingleThreadTickFunction))
		, RunnableThread(IsForkable == FThread::Forkable ? FForkProcessHelper::CreateForkableThread(this, ThreadName, StackSize, ThreadPriority, ThreadAffinity.ThreadAffinityMask)
			: FRunnableThread::Create(this, ThreadName, StackSize, ThreadPriority, ThreadAffinity.ThreadAffinityMask))
	{
		checkf(IsJoinable(), TEXT("Failed to create a thread. Check if it ran out of memory or a similar \"out of system resources\" issue"));
		RunnableThread->SetThreadAffinity(ThreadAffinity);
	}

	// Provides a reference to self. This can't be done by `SharedFromThis` in the constructor because the instance is not constructed yet
	void Initialize(const TSharedPtr<FThreadImpl, ESPMode::ThreadSafe>& InSelf)
	{
		Self = InSelf;
		bIsInitialized = true;
	}

	bool IsJoinable() const
	{
		return RunnableThread.IsValid() && FPlatformTLS::GetCurrentThreadId() != RunnableThread->GetThreadID();
	}

	void Join()
	{
		check(IsJoinable());

		RunnableThread->WaitForCompletion();
		RunnableThread.Reset();
	}

	uint32 GetThreadId() const
	{
		return RunnableThread.IsValid() ? RunnableThread->GetThreadID() : FThread::InvalidThreadId;
	}

private:
	uint32 Run() override
	{
		// the thread can be started before `RunnableThread` member is initialized
		//check(RunnableThread.IsValid());
		ThreadFunction();

		return 0;
	}

	void Exit() override
	{
		// busy-wait till `Self` is initialized before releasing it
		while (!bIsInitialized) {}

		check(Self.IsValid());

		// we're about to exit the thread. Release the reference to self. If the thread is detached, it's the only reference
		// and so this instance will be deleted. No member access should be performed after this point.
		Self.Reset();
	}

	virtual FSingleThreadRunnable* GetSingleThreadInterface() override
	{
		return this;
	}

	virtual void Tick() override
	{
		SingleThreadTickFunction();
	}

private:
	// Two strong references are hold for `FThreadImpl`, one in parent `FThread` and another here. The reference in `FThreadImpl` is released
	// on `Detach`, as to physically detach `FThreadImpl` from `FThread`. The reference below (`Self`) is released before exiting the thread,
	// as the work was done and `FThreadImpl` doesn't mind to be deleted. Releasing the last reference deletes the instance, so 
	// no member access can be performed after that.
	// This must be declared before `RunnableThread` so it's already initialized when the thread is created, otherwise the thread can complete
	// before `Self` is initialized.
	TSharedPtr<FThreadImpl, ESPMode::ThreadSafe> Self;

	TAtomic<bool> bIsInitialized{ false };

	TUniqueFunction<void()> ThreadFunction;
	TUniqueFunction<void()> SingleThreadTickFunction;
	TUniquePtr<FRunnableThread> RunnableThread;
};

FThread::FThread(
	TCHAR const* ThreadName,
	TUniqueFunction<void()>&& ThreadFunction,
	uint32 StackSize/* = 0*/,
	EThreadPriority ThreadPriority/* = TPri_Normal*/,
	FThreadAffinity ThreadAffinity/* = FThreadAffinity()*/,
	EForkable IsForkable/* = NonForkable*/
)
	: Impl(MakeShared<FThreadImpl, ESPMode::ThreadSafe>(ThreadName, MoveTemp(ThreadFunction), [] {}, StackSize, ThreadPriority, ThreadAffinity, IsForkable))
{
	Impl->Initialize(Impl);
}

FThread::FThread(
	TCHAR const* ThreadName,
	TUniqueFunction<void()>&& ThreadFunction,
	TUniqueFunction<void()>&& SingleThreadTickFunction,
	uint32 StackSize/* = 0*/,
	EThreadPriority ThreadPriority/* = TPri_Normal*/,
	FThreadAffinity ThreadAffinity/* = FThreadAffinity()*/,
	EForkable IsForkable/* = NonForkable*/
)
	: Impl(MakeShared<FThreadImpl, ESPMode::ThreadSafe>(ThreadName, MoveTemp(ThreadFunction), MoveTemp(SingleThreadTickFunction), StackSize, ThreadPriority, ThreadAffinity, IsForkable))
{
	Impl->Initialize(Impl);
}

FThread& FThread::operator=(FThread&& Other)
{
	checkf(!IsJoinable(), TEXT("Joinable thread cannot be assigned"));
	Impl = MoveTemp(Other.Impl);
	return *this;
}

FThread::~FThread()
{
	checkf(!Impl.IsValid(), TEXT("FThread must be either joined or detached before destruction"));
}

bool FThread::IsJoinable() const
{
	return Impl.IsValid() && Impl->IsJoinable();
}

void FThread::Join()
{
	check(Impl.IsValid());
	Impl->Join();
	Impl.Reset(); // no need to keep this memory anymore (it's the last reference), and this also simplifies the check in the destructor
}

uint32 FThread::GetThreadId() const
{
	return Impl.IsValid() ? Impl->GetThreadId() : FThread::InvalidThreadId;
}

#if 0 // disabled as it doesn't work as intended

void FThread::Detach()
{
	check(Impl.IsValid());
	Impl.Reset(); // releases this reference, `FThreadImpl` instance can still be alive until the thread finishes its execution
}

#endif