// Copyright Epic Games, Inc. All Rights Reserved.
#include "HAL/ThreadHeartBeat.h"
#include "HAL/PlatformStackWalk.h"
#include "HAL/PlatformTime.h"
#include "HAL/PlatformProcess.h"
#include "Logging/LogMacros.h"
#include "CoreGlobals.h"
#include "HAL/RunnableThread.h"
#include "HAL/ThreadManager.h"
#include "Misc/ScopeLock.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CommandLine.h"
#include "Misc/CoreDelegates.h"
#include "HAL/ExceptionHandling.h"
#include "GenericPlatform/GenericPlatformCrashContext.h"
#include "Stats/Stats.h"
#include "Async/TaskGraphInterfaces.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "Misc/App.h"
#include "Misc/Fork.h"
#include "AutoRTFM.h"

// When enabled, the heart beat thread will call abort() when a hang
// is detected, rather than performing stack back-traces and logging.
#define MINIMAL_FATAL_HANG_DETECTION	(PLATFORM_USE_MINIMAL_HANG_DETECTION && 1)

#ifndef UE_ASSERT_ON_HANG
	#define UE_ASSERT_ON_HANG 0
#endif

// The maximum clock time steps for the hang and hitch detectors.
// These are the amounts the clocks are allowed to advance by before another tick is required.
const double HangDetectorClock_MaxTimeStep_MS = 2000.0;
const double HitchDetectorClock_MaxTimeStep_MS = 50.0;

extern CORE_API bool GIsGPUCrashed;

FThreadHeartBeatClock::FThreadHeartBeatClock(double InMaxTimeStep)
	: MaxTimeStepCycles(FPlatformTime::SecondsToCycles64(InMaxTimeStep))
{
	CurrentCycles = FPlatformTime::Cycles64();
	LastRealTickCycles = CurrentCycles;
}

void FThreadHeartBeatClock::Tick()
{
	uint64 CurrentRealTickCycles = FPlatformTime::Cycles64();
	uint64 DeltaCycles = CurrentRealTickCycles - LastRealTickCycles;
	uint64 ClampedCycles = FMath::Min(DeltaCycles, MaxTimeStepCycles);

	CurrentCycles += ClampedCycles;
	LastRealTickCycles = CurrentRealTickCycles;
}

double FThreadHeartBeatClock::Seconds()
{
	uint64 Offset = FPlatformTime::Cycles64() - LastRealTickCycles;
	uint64 ClampedOffset = FMath::Min(Offset, MaxTimeStepCycles);
	uint64 CyclesPerSecond = FPlatformTime::SecondsToCycles64(1.0);
	uint64 Cycles = CurrentCycles + ClampedOffset;
	uint64 Seconds = Cycles / CyclesPerSecond;
	uint64 RemainderCycles = Cycles % CyclesPerSecond;

	return (double)Seconds + FPlatformTime::ToSeconds64(RemainderCycles);
}

FThreadHeartBeat::FThreadHeartBeat()
	: Thread(nullptr)
	, bReadyToCheckHeartbeat(false)
	, ConfigHangDuration(0)
	, CurrentHangDuration(0)
	, ConfigPresentDuration(0)
	, CurrentPresentDuration(0)
	, ConfigStuckDuration(0)
	, CurrentStuckDuration(0)
	, HangDurationMultiplier(1.0)
	, LastHangCallstackCRC(0)
	, LastHungThreadId(InvalidThreadId)
	, LastStuckThreadId(InvalidThreadId)
	, bHangsAreFatal(false)
	, GlobalSuspendCount(0)
	, CheckpointSuspendCount(0)
	, Clock(HangDetectorClock_MaxTimeStep_MS / 1000)
{
	// Start with the frame-present based hang detection disabled. This will be automatically enabled on
	// platforms that implement frame-present based detection on the first call the PresentFrame().
	PresentHeartBeat.SuspendedCount = 1;

	InitSettingsInternal();

	const bool bAllowThreadHeartBeat = FPlatformMisc::AllowThreadHeartBeat() && (ConfigHangDuration > 0.0 || ConfigPresentDuration > 0.0);

	// We don't care about programs for now so no point in spawning the extra thread
#if USE_HANG_DETECTION
	if (bAllowThreadHeartBeat && FPlatformProcess::SupportsMultithreading())
	{
		Thread = FRunnableThread::Create(this, TEXT("FHeartBeatThread"), 0, TPri_AboveNormal);
	}
#endif

	if (!bAllowThreadHeartBeat)
	{
		// Disable the check
		ConfigHangDuration = 0.0;
		ConfigPresentDuration = 0.0;
	}
}

FThreadHeartBeat::~FThreadHeartBeat()
{
	if (Thread)
	{
		delete Thread;
		Thread = nullptr;
	}
}

FThreadHeartBeat* FThreadHeartBeat::Singleton = nullptr;

FThreadHeartBeat& FThreadHeartBeat::Get()
{
	struct FInitHelper
	{
		FThreadHeartBeat* Instance;

		FInitHelper()
		{
			check(!Singleton);
			Instance = new FThreadHeartBeat();
			Singleton = Instance;
		}

		~FInitHelper()
		{
			Singleton = nullptr;

			delete Instance;
			Instance = nullptr;
		}
	};

	// Use a function static helper to ensure creation
	// of the FThreadHeartBeat instance is thread safe.
	static FInitHelper Helper;
	return *Helper.Instance;
}

FThreadHeartBeat* FThreadHeartBeat::GetNoInit()
{
	return Singleton;
}

//~ Begin FRunnable Interface.
bool FThreadHeartBeat::Init()
{
	return true;
}

void FORCENOINLINE FThreadHeartBeat::OnPresentHang(double HangDuration)
{
#if MINIMAL_FATAL_HANG_DETECTION

	LastHungThreadId = FThreadHeartBeat::PresentThreadId;
	FGenericCrashContext::SetEngineData(TEXT("HungThread"), TEXT("Present"));

	// We want to avoid all memory allocations if a hang is detected.
	// Force a crash in a way that will generate a crash report.

	// Delegate implementation will be called from the hang detector thread and not from the hung thread
	OnHangDelegate.ExecuteIfBound(FThreadHeartBeat::PresentThreadId);

	// Avoiding calling RaiseException here will keep OnPresentHang on the top of the crash callstack,
	// making crash bucketing easier when looking at retail crash dumps on supported platforms.

	//FPlatformMisc::RaiseException(0xe0000002);
	*((uint32*)3) = 0xe0000002;

#elif UE_ASSERT_ON_HANG
	UE_LOGF(LogCore, Fatal, "Frame present hang detected. A frame has not been presented for %.2f seconds.", HangDuration);
#else
	UE_LOGF(LogCore, Error, "Frame present hang detected. A frame has not been presented for %.2f seconds.", HangDuration);
#endif
}

void FORCENOINLINE FThreadHeartBeat::OnHang(double HangDuration, uint32 ThreadThatHung)
{
#if MINIMAL_FATAL_HANG_DETECTION

	LastHungThreadId = ThreadThatHung;
	FGenericCrashContext::SetEngineData(TEXT("HungThread"), LexToString(ThreadThatHung));

	// Delegate implementation will be called from the hang detector thread and not from the hung thread
	OnHangDelegate.ExecuteIfBound(ThreadThatHung);

	// We want to avoid all memory allocations if a hang is detected.
	// Force a crash in a way that will generate a crash report.

	// Avoiding calling RaiseException here will keep OnPresentHang on the top of the crash callstack,
	// making crash bucketing easier when looking at retail crash dumps on supported platforms.

	//FPlatformMisc::RaiseException(0xe0000001);
	*((uint32*)3) = 0xe0000001;

#else // MINIMAL_FATAL_HANG_DETECTION == 0

	// Capture the stack in the thread that hung
	static const int32 MaxStackFrames = 100;
	uint64 StackFrames[MaxStackFrames];
	int32 NumStackFrames = FPlatformStackWalk::CaptureThreadStackBackTrace(ThreadThatHung, StackFrames, MaxStackFrames);

	// First verify we're not reporting the same hang over and over again
	uint32 CallstackCRC = FCrc::MemCrc32(StackFrames, sizeof(StackFrames[0]) * NumStackFrames);
	if (CallstackCRC != LastHangCallstackCRC || ThreadThatHung != LastHungThreadId)
	{
		LastHangCallstackCRC = CallstackCRC;
		LastHungThreadId = ThreadThatHung;

		// Convert the stack trace to text
		TArray<FString> StackLines;
		for (int32 Idx = 0; Idx < NumStackFrames; Idx++)
		{
			ANSICHAR Buffer[1024];
			Buffer[0] = '\0';
			FPlatformStackWalk::ProgramCounterToHumanReadableString(Idx, StackFrames[Idx], Buffer, sizeof(Buffer));
			StackLines.Add(Buffer);
		}

		// Dump the callstack and the thread name to log
		FString ThreadName = FThreadManager::GetThreadName(ThreadThatHung);
		if (ThreadName.IsEmpty())
		{
			ThreadName = FString::Printf(TEXT("unknown thread (%u)"), ThreadThatHung);
		}
		UE_LOGF(LogCore, Error, "Hang detected on %ls (thread hasn't sent a heartbeat for %.2f seconds):", *ThreadName, HangDuration);
		for (int32 Idx = 0; Idx < StackLines.Num(); Idx++)
		{
			UE_LOGF(LogCore, Error, "  %ls", *StackLines[Idx]);
		}

		// Assert (on the current thread unfortunately) with a trimmed stack.
		FString StackTrimmed;
		for (int32 LineIndex = 0; LineIndex < StackLines.Num() && StackTrimmed.Len() < 512; ++LineIndex)
		{
			StackTrimmed += TEXT("  ");
			StackTrimmed += StackLines[LineIndex];
			StackTrimmed += LINE_TERMINATOR;
		}

		const FString ErrorMessage = FString::Printf(TEXT("Hang detected on %s:%s%s%sCheck log for full callstack."), *ThreadName, LINE_TERMINATOR, *StackTrimmed, LINE_TERMINATOR);

#if PLATFORM_USE_REPORT_ENSURE
		UE_LOGF(LogCore, Error, "%ls", *ErrorMessage);

		if (bHangsAreFatal)
		{
			GLog->Panic();
			FCoreDelegates::OnHandleSystemHang.Broadcast();
		}
		else
		{
			GLog->FlushThreadedLogs(EOutputDeviceRedirectorFlushOptions::Async);
		}

		// Skip macros and FDebug, we always want this to fire
		ReportHang(*ErrorMessage, StackFrames, NumStackFrames, ThreadThatHung);

		if (bHangsAreFatal)
		{
			if (FApp::CanEverRender())
			{
				FPlatformMisc::MessageBoxExt(EAppMsgType::Ok,
					*NSLOCTEXT("MessageDialog", "ReportHangError_Body", "The application has hung and will now close. We apologize for the inconvenience.").ToString(),
					*NSLOCTEXT("MessageDialog", "ReportHangError_Title", "Application Hang Detected").ToString());
			}

			FPlatformMisc::RequestExit(true, TEXT("FThreadHeartbeat::OnHang"));
		}
#else
		if (bHangsAreFatal)
		{
			UE_LOGF(LogCore, Fatal, "%ls", *ErrorMessage);
		}
		else
		{
			UE_LOGF(LogCore, Error, "%ls", *ErrorMessage);
		}
#endif
	}
#endif // MINIMAL_FATAL_HANG_DETECTION == 0
}

void FORCENOINLINE FThreadHeartBeat::OnGPUHang(double HangDuration)
{
#if MINIMAL_FATAL_HANG_DETECTION
	// We want to avoid all memory allocations if a hang is detected.
	// Force a crash in a way that will generate a crash report.

	// Avoiding calling RaiseException here will keep OnGPUHang on the top of the crash callstack,
	// making crash bucketing easier when looking at retail crash dumps on supported platforms.

	//FPlatformMisc::RaiseException(0xe0000003);
	*((uint32*)3) = 0xe0000003;

#elif UE_ASSERT_ON_HANG
	UE_LOGF(LogCore, Fatal, "GPU hang detected. A frame has not been presented for %.2f seconds.", HangDuration);
#else
	UE_LOGF(LogCore, Error, "GPU hang detected. A frame has not been presented for %.2f seconds.", HangDuration);
#endif
}

uint32 FThreadHeartBeat::Run()
{
	bool InHungState = false;

#if USE_HANG_DETECTION
	while (StopTaskCounter.GetValue() == 0 && !IsEngineExitRequested())
	{
		double HangDuration;
		uint32 ThreadThatHung = CheckHeartBeat(HangDuration);
		if (ThreadThatHung == FThreadHeartBeat::InvalidThreadId)
		{
			ThreadThatHung = CheckFunctionHeartBeat(HangDuration);
		}

		if (ThreadThatHung == FThreadHeartBeat::InvalidThreadId)
		{
			ThreadThatHung = CheckCheckpointHeartBeat(HangDuration);
		}

		if (ThreadThatHung == FThreadHeartBeat::InvalidThreadId)
		{
			InHungState = false;
		}
		else if (InHungState == false)
		{
			// Only want to call this once per hang (particularly if we're just ensuring).
			InHungState = true;

			if (GIsGPUCrashed)
			{ 
				OnGPUHang(HangDuration);
			}
			else if (ThreadThatHung == FThreadHeartBeat::PresentThreadId)
			{
				OnPresentHang(HangDuration);
			}
			else
			{
				OnHang(HangDuration, ThreadThatHung);
			}
		}

		if (StopTaskCounter.GetValue() == 0 && !IsEngineExitRequested())
		{
			FPlatformProcess::SleepNoStats(0.5f);
		}

		Clock.Tick();
	}
#endif // USE_HANG_DETECTION

	return 0;
}

void FThreadHeartBeat::Stop()
{
	bReadyToCheckHeartbeat = false;
	StopTaskCounter.Increment();
}

void FThreadHeartBeat::Start()
{
	bReadyToCheckHeartbeat = true;
}

void FThreadHeartBeat::InitSettings()
{
	const uint32 ThreadId = FPlatformTLS::GetCurrentThreadId();
	if (ThreadId == GGameThreadId && GConfig)
	{
		FScopeLock HeartBeatLock(&HeartBeatCritical);
		InitSettingsInternal();
	}
}

void FThreadHeartBeat::InitSettingsInternal()
{
	double NewStuckDuration = 1.0;

	// Default to 25 seconds if not overridden in config.
	double NewHangDuration = 25.0;
	double NewPresentDuration = 0.0;

	bool bNewHangsAreFatal = !!(UE_ASSERT_ON_HANG);

	if (GConfig)
	{
		GConfig->GetDouble(TEXT("Core.System"), TEXT("StuckDuration"), NewStuckDuration, GEngineIni);
		GConfig->GetDouble(TEXT("Core.System"), TEXT("HangDuration"), NewHangDuration, GEngineIni);
		GConfig->GetDouble(TEXT("Core.System"), TEXT("PresentHangDuration"), NewPresentDuration, GEngineIni);
		GConfig->GetBool(TEXT("Core.System"), TEXT("HangsAreFatal"), bNewHangsAreFatal, GEngineIni);

		const double MinStuckDuration = 1.0;
		if (NewStuckDuration > 0.0 && NewStuckDuration < MinStuckDuration)
		{
			UE_LOGF(LogCore, Warning, "StuckDuration is set to %.4fs which is a very short time for stuck detection. Changing to %.2fs.", NewStuckDuration, MinStuckDuration);
			NewStuckDuration = MinStuckDuration;
		}

		const double MinHangDuration = 5.0;
		if (NewHangDuration > 0.0 && NewHangDuration < MinHangDuration)
		{
			UE_LOGF(LogCore, Warning, "HangDuration is set to %.4fs which is a very short time for hang detection. Changing to %.2fs.", NewHangDuration, MinHangDuration);
			NewHangDuration = MinHangDuration;
		}

		const double MinPresentDuration = 5.0;
		if (NewPresentDuration > 0.0 && NewPresentDuration < MinPresentDuration)
		{
			UE_LOGF(LogCore, Warning, "PresentHangDuration is set to %.4fs which is a very short time for hang detection. Changing to %.2fs.", NewPresentDuration, MinPresentDuration);
			NewPresentDuration = MinPresentDuration;
		}
	}

	ConfigStuckDuration = NewStuckDuration;
	CurrentStuckDuration = ConfigStuckDuration;

	ConfigHangDuration = NewHangDuration;
	ConfigPresentDuration = NewPresentDuration;

	CurrentHangDuration = ConfigHangDuration * HangDurationMultiplier;
	CurrentPresentDuration = ConfigPresentDuration * HangDurationMultiplier;

	bHangsAreFatal = bNewHangsAreFatal;

	// Update the existing thread and present hang durations.
	// Only increase existing thread's heartbeats.
	// We don't want to decrease here, threads and present logic will pick up a smaller hang duration 
	// the next time they call HeartBeat() or PresentFrame().
	for (TPair<uint32, FHeartBeatInfo>& Pair : ThreadHeartBeat)
	{
		if (Pair.Value.HangDuration < CurrentHangDuration)
		{
			Pair.Value.HangDuration = CurrentHangDuration;
		}
	}
	
	if (PresentHeartBeat.HangDuration < CurrentPresentDuration)
	{
		PresentHeartBeat.HangDuration = CurrentPresentDuration;
	}
}

void FThreadHeartBeat::HeartBeat(bool bReadConfig)
{
#if USE_HANG_DETECTION
	// disable on platforms that don't start the thread
	if (FPlatformMisc::AllowThreadHeartBeat() == false)
	{
		return;
	}

	uint32 ThreadId = FPlatformTLS::GetCurrentThreadId();
	FScopeLock HeartBeatLock(&HeartBeatCritical);
	if (bReadConfig && ThreadId == GGameThreadId && GConfig)
	{
		InitSettingsInternal();
	}
	FHeartBeatInfo& HeartBeatInfo = ThreadHeartBeat.FindOrAdd(ThreadId);
	HeartBeatInfo.LastHeartBeatTime = Clock.Seconds();
	HeartBeatInfo.HangDuration = CurrentHangDuration;
	HeartBeatInfo.StuckDuration = CurrentStuckDuration;
#endif
}

void FThreadHeartBeat::HeartBeat()
{
#if USE_HANG_DETECTION
	// disable on platforms that don't start the thread
	if (FPlatformMisc::AllowThreadHeartBeat() == false)
	{
		return;
	}

	uint32 ThreadId = FPlatformTLS::GetCurrentThreadId();
	FScopeLock HeartBeatLock(&HeartBeatCritical);

	FHeartBeatInfo& HeartBeatInfo = ThreadHeartBeat.FindOrAdd(ThreadId);
	HeartBeatInfo.LastHeartBeatTime = Clock.Seconds();
	HeartBeatInfo.HangDuration = CurrentHangDuration;
	HeartBeatInfo.StuckDuration = CurrentStuckDuration;
#endif
}

void FThreadHeartBeat::PresentFrame()
{
#if USE_HANG_DETECTION
	FScopeLock HeartBeatLock(&HeartBeatCritical);
	PresentHeartBeat.LastHeartBeatTime = Clock.Seconds();
	PresentHeartBeat.HangDuration = CurrentPresentDuration;

	static bool bFirst = true;
	if (bFirst)
	{
		// Decrement the suspend count on the first call to PresentFrame.
		// This enables frame-present based hang detection on supported platforms.
		PresentHeartBeat.SuspendedCount--;
		bFirst = false;
	}
#endif
}

void FThreadHeartBeat::MonitorFunctionStart()
{
#if USE_HANG_DETECTION
	// disable on platforms that don't start the thread
	if (FPlatformMisc::AllowThreadHeartBeat() == false)
	{
		return;
	}

	uint32 ThreadId = FPlatformTLS::GetCurrentThreadId();
	FScopeLock FunctionHeartBeatLock(&FunctionHeartBeatCritical);

	FHeartBeatInfo& HeartBeatInfo = FunctionHeartBeat.FindOrAdd(ThreadId);
	HeartBeatInfo.LastHeartBeatTime = Clock.Seconds();
	HeartBeatInfo.HangDuration = CurrentHangDuration;
	HeartBeatInfo.SuspendedCount = 0;
#endif
}

void FThreadHeartBeat::MonitorFunctionEnd()
{
#if USE_HANG_DETECTION
	// disable on platforms that don't start the thread
	if (FPlatformMisc::AllowThreadHeartBeat() == false)
	{
		return;
	}
	uint32 ThreadId = FPlatformTLS::GetCurrentThreadId();
	FScopeLock FunctionHeartBeatLock(&FunctionHeartBeatCritical);
	FHeartBeatInfo* HeartBeatInfo = FunctionHeartBeat.Find(ThreadId);
	if (HeartBeatInfo != nullptr)
	{
		HeartBeatInfo->SuspendedCount = 1;
	}
	else
	{
		// has to have been there otherwise the Start/End are out of order/unbalanced
		check(0);
	}
#endif
}


bool FThreadHeartBeat::IsEnabled()
{
	bool CheckBeats = false;
#if USE_HANG_DETECTION
	static bool bForceEnabled = FParse::Param(FCommandLine::Get(), TEXT("debughangdetection"));
	static bool bDisabled = !bForceEnabled && FParse::Param(FCommandLine::Get(), TEXT("nothreadtimeout"));

	CheckBeats = (ConfigHangDuration > 0.0 || ConfigPresentDuration > 0.0)
		&& bReadyToCheckHeartbeat
		&& !IsEngineExitRequested()
		&& (bForceEnabled || !FPlatformMisc::IsDebuggerPresent())
		&& !bDisabled
		&& !GlobalSuspendCount.GetValue();
#endif
	return CheckBeats;
}


uint32 FThreadHeartBeat::CheckHeartBeat(double& OutHangDuration)
{
	// Editor and debug builds run too slow to measure them correctly
#if USE_HANG_DETECTION
	bool CheckBeats = IsEnabled();

	if (CheckBeats)
	{
		const double CurrentTime = Clock.Seconds();
		FScopeLock HeartBeatLock(&HeartBeatCritical);

		if (ConfigHangDuration > 0.0)
		{
			uint32 LongestStuckThreadId = InvalidThreadId;
			double LongestStuckThreadStuckTime = 0.0;

			// Check heartbeat for all threads and return thread ID of the thread that hung.
			// Note: We only return a thread id for a thread that has updated since the last hang, i.e. is still alive
			// This avoids the case where a user may be in a deep and minorly varying callstack and flood us with reports
			for (TPair<uint32, FHeartBeatInfo>& LastHeartBeat : ThreadHeartBeat)
			{
				FHeartBeatInfo& HeartBeatInfo = LastHeartBeat.Value;
				if (HeartBeatInfo.SuspendedCount == 0)
				{
					double TimeSinceLastHeartbeat = (CurrentTime - HeartBeatInfo.LastHeartBeatTime);

					if (TimeSinceLastHeartbeat > HeartBeatInfo.HangDuration && HeartBeatInfo.LastHeartBeatTime >= HeartBeatInfo.LastHangTime)
					{
						HeartBeatInfo.LastHangTime = CurrentTime;
						OutHangDuration = HeartBeatInfo.HangDuration;
						return LastHeartBeat.Key;
					}
					else if (HeartBeatInfo.LastHeartBeatTime >= HeartBeatInfo.LastStuckTime)
					{
						// Are we considered stuck?
						if (TimeSinceLastHeartbeat > HeartBeatInfo.StuckDuration)
						{
							// Are we stuck longer than another thread (maybe boosting them stuck us).
							if ((LastStuckThreadId == InvalidThreadId) || (CurrentTime - HeartBeatInfo.LastHeartBeatTime) > LongestStuckThreadStuckTime)
							{
								LongestStuckThreadId = LastHeartBeat.Key;
								LongestStuckThreadStuckTime = CurrentTime - HeartBeatInfo.LastHeartBeatTime;
							}
						}
						else if (LastHeartBeat.Key == LastStuckThreadId)
						{
							// We we're stuck but now we're not.
							OnUnstuck.ExecuteIfBound(LastStuckThreadId);
							LastStuckThreadId = InvalidThreadId;
						}
					}
				}
				else if (LastHeartBeat.Key == LastStuckThreadId)
				{
					// We're not checking so clean up any existing stuck thread action.
					OnUnstuck.ExecuteIfBound(LastStuckThreadId);
					LastStuckThreadId = InvalidThreadId;
				}
			}

			if (LongestStuckThreadId != InvalidThreadId)
			{
				// Is there a currently stuck thread. Replace it.
				if (LastStuckThreadId != LongestStuckThreadId)
				{
					OnUnstuck.ExecuteIfBound(LastStuckThreadId);
				}

				// Notify and note stuck thread.
				LastStuckThreadId = LongestStuckThreadId;
				ThreadHeartBeat[LastStuckThreadId].LastStuckTime = CurrentTime;
				OnStuck.ExecuteIfBound(LastStuckThreadId);
			}

		}

		if (ConfigPresentDuration > 0.0)
		{
			if (PresentHeartBeat.SuspendedCount == 0 && (CurrentTime - PresentHeartBeat.LastHeartBeatTime) > PresentHeartBeat.HangDuration)
			{
				// Frames are no longer presenting.
				PresentHeartBeat.LastHeartBeatTime = CurrentTime;
				OutHangDuration = PresentHeartBeat.HangDuration;
				return PresentThreadId;
			}
		}
	}

#endif
	return InvalidThreadId;
}

uint32 FThreadHeartBeat::CheckFunctionHeartBeat(double& OutHangDuration)
{
	// Editor and debug builds run too slow to measure them correctly
#if USE_HANG_DETECTION
	bool CheckBeats = IsEnabled();

	if (CheckBeats)
	{
		const double CurrentTime = Clock.Seconds();
		FScopeLock HeartBeatLock(&FunctionHeartBeatCritical);
		if (ConfigHangDuration > 0.0)
		{
			// Check heartbeat for all functions and return thread ID of the thread that was running the function when it hung.
			// Note: We only return a thread id for a thread that has updated since the last hang, i.e. is still alive
			// This avoids the case where a user may be in a deep and minorly varying callstack and flood us with reports
			for (TPair<uint32, FHeartBeatInfo>& LastHeartBeat : FunctionHeartBeat)
			{
				FHeartBeatInfo& HeartBeatInfo = LastHeartBeat.Value;
				if (HeartBeatInfo.SuspendedCount == 0 && (CurrentTime - HeartBeatInfo.LastHeartBeatTime) > HeartBeatInfo.HangDuration && HeartBeatInfo.LastHeartBeatTime >= HeartBeatInfo.LastHangTime)
				{
					HeartBeatInfo.LastHangTime = CurrentTime;
					OutHangDuration = HeartBeatInfo.HangDuration;
					return LastHeartBeat.Key;
				}
			}
		}
	}
#endif
	return InvalidThreadId;
}

void FThreadHeartBeat::MonitorCheckpointStart(FName EndCheckpoint, double TimeToReachCheckpoint)
{
#if USE_HANG_DETECTION
	// disable on platforms that don't start the thread
	if (FPlatformMisc::AllowThreadHeartBeat() == false)
	{
		return;
	}

	FScopeLock CheckpointHeartBeatLock(&CheckpointHeartBeatCritical);

	if (CheckpointHeartBeat.Find(EndCheckpoint) == nullptr)
	{
		FHeartBeatInfo& HeartBeatInfo = CheckpointHeartBeat.Add(EndCheckpoint);
		HeartBeatInfo.LastHeartBeatTime = Clock.Seconds();
		HeartBeatInfo.HangDuration = TimeToReachCheckpoint;
		HeartBeatInfo.HeartBeatName = EndCheckpoint;
		HeartBeatInfo.SuspendedCount = CheckpointSuspendCount;
	}
#endif
}

void FThreadHeartBeat::MonitorCheckpointEnd(FName Checkpoint)
{
#if USE_HANG_DETECTION
	// disable on platforms that don't start the thread
	if (FPlatformMisc::AllowThreadHeartBeat() == false)
	{
		return;
	}
	FScopeLock CheckpointHeartBeatLock(&CheckpointHeartBeatCritical);
	CheckpointHeartBeat.Remove(Checkpoint);
#endif
}

uint32 FThreadHeartBeat::CheckCheckpointHeartBeat(double& OutHangDuration)
{
	// Editor and debug builds run too slow to measure them correctly
#if USE_HANG_DETECTION
	static bool bCheckpointDisabled = FParse::Param(FCommandLine::Get(), TEXT("nocheckpointhangdetector"));
	bool CheckBeats = IsEnabled() && !bCheckpointDisabled;

	if (CheckBeats)
	{
		const double CurrentTime = Clock.Seconds();
		FScopeLock HeartBeatLock(&CheckpointHeartBeatCritical);
		if (ConfigHangDuration > 0.0)
		{
			// Check heartbeat for all checkpoints and return thread ID of the thread that initally marked the checkpoint when it hung.
			// Note: We only return a thread id for a thread that has updated since the last hang, i.e. is still alive
			// This avoids the case where a user may be in a deep and minorly varying callstack and flood us with reports
			for (TPair<FName, FHeartBeatInfo>& LastHeartBeat : CheckpointHeartBeat)
			{
				FHeartBeatInfo& HeartBeatInfo = LastHeartBeat.Value;
				if (HeartBeatInfo.SuspendedCount == 0 && (CurrentTime - HeartBeatInfo.LastHeartBeatTime > HeartBeatInfo.HangDuration && HeartBeatInfo.LastHeartBeatTime >= HeartBeatInfo.LastHangTime))
				{
					if (HeartBeatInfo.HangDuration > 0.0)
					{
						UE_LOGF(LogCore, Warning, "Failed to reach checkpoint: '%ls' within alotted time of %.2f. Triggering hang detector.", *HeartBeatInfo.HeartBeatName.ToString(), HeartBeatInfo.HangDuration);

						HeartBeatInfo.LastHangTime = CurrentTime;
						OutHangDuration = HeartBeatInfo.HangDuration;
						LastHungThreadId = FPlatformTLS::GetCurrentThreadId();

						*((uint32*)3) = 0xe0000001;

						return 0;
					}
				}
			}
		}
	}
#endif
	return InvalidThreadId;
}

void FThreadHeartBeat::KillHeartBeat()
{
#if USE_HANG_DETECTION
	uint32 ThreadId = FPlatformTLS::GetCurrentThreadId();
	FScopeLock HeartBeatLock(&HeartBeatCritical);
	ThreadHeartBeat.Remove(ThreadId);
#endif
}

void FThreadHeartBeat::SuspendHeartBeat(bool bAllThreads)
{
#if USE_HANG_DETECTION	
	{
		FScopeLock HeartBeatLock(&HeartBeatCritical);
		if (bAllThreads)
		{
			GlobalSuspendCount.Increment();
		}
		else
		{
			uint32 ThreadId = FPlatformTLS::GetCurrentThreadId();
			FHeartBeatInfo* HeartBeatInfo = ThreadHeartBeat.Find(ThreadId);
			if (HeartBeatInfo)
			{
				HeartBeatInfo->Suspend();
			}
		}

		// Suspend the frame-present based detection at the same time.
		PresentHeartBeat.SuspendedCount++;
	}

	// Suspend the checkpoint heartbeats
	{
		if (!bAllThreads)
		{
			FScopeLock HeartBeatLock(&CheckpointHeartBeatCritical);
			check(CheckpointSuspendCount >= 0);
			++CheckpointSuspendCount;
			for (TPair<FName, FHeartBeatInfo>& HeartBeatEntry : CheckpointHeartBeat)
			{
				HeartBeatEntry.Value.Suspend();
			}
		}
	}
#endif
}


void FThreadHeartBeat::ResumeHeartBeat(bool bAllThreads)
{
#if USE_HANG_DETECTION	
	bool bLastThreadResumed = false;
	{
		FScopeLock HeartBeatLock(&HeartBeatCritical);

		const double CurrentTime = Clock.Seconds();
		if (bAllThreads)
		{
			if (GlobalSuspendCount.GetValue() == 0)
			{
				// Resume without matching Suspend, ignore it
				return;
			}
		
			if (GlobalSuspendCount.Decrement() == 0)
			{
				bLastThreadResumed = true;
				for (TPair<uint32, FHeartBeatInfo>& HeartBeatEntry : ThreadHeartBeat)
				{
					HeartBeatEntry.Value.LastHeartBeatTime = CurrentTime;
				}
			}
		}
		else
		{
			uint32 ThreadId = FPlatformTLS::GetCurrentThreadId();
			FHeartBeatInfo* HeartBeatInfo = ThreadHeartBeat.Find(ThreadId);
			if (HeartBeatInfo)
			{
				HeartBeatInfo->Resume(CurrentTime);
			}
		}
		// Resume the frame-present based detection at the same time.
		PresentHeartBeat.SuspendedCount--;
		PresentHeartBeat.LastHeartBeatTime = Clock.Seconds();
	}

	// Resume the checkpoint heartbeats
	{
		const double CurrentTime = Clock.Seconds();
		FScopeLock HeartBeatLock(&CheckpointHeartBeatCritical);
		if (bAllThreads)
		{
			if (bLastThreadResumed)
			{
				for (TPair<FName, FHeartBeatInfo>& HeartBeatEntry : CheckpointHeartBeat)
				{
					HeartBeatEntry.Value.LastHeartBeatTime = CurrentTime;
				}
			}
		}
		else
		{
			check(CheckpointSuspendCount > 0);
			CheckpointSuspendCount--;
			for (TPair<FName, FHeartBeatInfo>& HeartBeatEntry : CheckpointHeartBeat)
			{
				HeartBeatEntry.Value.Resume(CurrentTime);
			}
		}
	}
#endif
}

bool FThreadHeartBeat::IsBeating()
{
	if (GlobalSuspendCount.GetValue() == 0)
	{
		uint32 ThreadId = FPlatformTLS::GetCurrentThreadId();
		FScopeLock HeartBeatLock(&HeartBeatCritical);
		FHeartBeatInfo* HeartBeatInfo = ThreadHeartBeat.Find(ThreadId);
		if (HeartBeatInfo && HeartBeatInfo->SuspendedCount == 0)
		{
			return true;
		}
	}
	return false;
}

void FThreadHeartBeat::SetDurationMultiplier(double NewMultiplier)
{
	check(IsInGameThread());

#if USE_HANG_DETECTION
	if (NewMultiplier < 1.0)
	{
		UE_LOGF(LogCore, Warning, "Cannot set the hang duration multiplier to less than 1.0. Specified value was %.4fs.", NewMultiplier);
		NewMultiplier = 1.0;
	}

	FScopeLock HeartBeatLock(&HeartBeatCritical);

	HangDurationMultiplier = NewMultiplier;
	InitSettingsInternal();

	UE_LOGF(LogCore, Display, "Setting hang detector multiplier to %.4fs. New hang duration: %.4fs. New present duration: %.4fs.", NewMultiplier, CurrentHangDuration, CurrentPresentDuration);
#endif
}

FGameThreadHitchHeartBeatThreaded::FGameThreadHitchHeartBeatThreaded()
	: Thread(nullptr)
#if USE_HITCH_DETECTION
	, bWalkStackOnHitch(false)
	, FirstStartTime(0.0)
	, FrameStartTime(0.0)
	, SuspendedCount(0)
#endif
	, Clock(HitchDetectorClock_MaxTimeStep_MS / 1000.0)
{
	// We don't care about programs for now so no point in spawning the extra thread
#if USE_HITCH_DETECTION
	InitSettingsInternal();
#endif
}

FGameThreadHitchHeartBeatThreaded::~FGameThreadHitchHeartBeatThreaded()
{
	if (Thread)
	{
		delete Thread;
		Thread = nullptr;
	}
}

FGameThreadHitchHeartBeatThreaded* FGameThreadHitchHeartBeatThreaded::Singleton = nullptr;

FGameThreadHitchHeartBeatThreaded& FGameThreadHitchHeartBeatThreaded::Get()
{
	struct FInitHelper
	{
		FGameThreadHitchHeartBeatThreaded* Instance;

		FInitHelper()
		{
			check(!Singleton);
			Instance = new FGameThreadHitchHeartBeatThreaded();
			Singleton = Instance;
		}

		~FInitHelper()
		{
			Singleton = nullptr;

			delete Instance;
			Instance = nullptr;
		}
	};

	// Use a function static helper to ensure creation
	// of the FGameThreadHitchHeartBeatThreaded instance is thread safe.
	static FInitHelper Helper;
	return *Helper.Instance;
}

FGameThreadHitchHeartBeatThreaded* FGameThreadHitchHeartBeatThreaded::GetNoInit()
{
	return Singleton;
}

//~ Begin FRunnable Interface.
bool FGameThreadHitchHeartBeatThreaded::Init()
{
	StopTaskCounter.Reset();
	return true;
}

void FGameThreadHitchHeartBeatThreaded::InitSettings()
{
#if USE_HITCH_DETECTION
	FScopeLock HeartBeatLock(&HeartBeatCritical);
	InitSettingsInternal();
#endif
}

void FGameThreadHitchHeartBeatThreaded::InitSettingsInternal()
{
#if USE_HITCH_DETECTION
	static bool bFirst = true;
	static bool bHasCmdLine = false;
	static float CmdLine_HangDuration = 0.0f;
	static bool CmdLine_StackWalk = false;

	float OldHangDuration = HangDuration;

	if (bFirst)
	{
		bHasCmdLine = FParse::Value(FCommandLine::Get(), TEXT("hitchdetection="), CmdLine_HangDuration);
		CmdLine_StackWalk = FParse::Param(FCommandLine::Get(), TEXT("hitchdetectionstackwalk"));

		// Determine whether to start suspended
		bStartSuspended = false;
		if (GConfig)
		{
			GConfig->GetBool(TEXT("Core.System"), TEXT("GameThreadHeartBeatStartSuspended"), bStartSuspended, GEngineIni);
		}
		if (FParse::Param(FCommandLine::Get(), TEXT("hitchdetectionstartsuspended")))
		{
			bStartSuspended = true;
		}
		else if (FParse::Param(FCommandLine::Get(), TEXT("hitchdetectionstartrunning")))
		{
			bStartSuspended = false;
		}
		if (bStartSuspended)
		{
			UE_LOGF(LogCore, Display, "Starting with HitchHeartbeat suspended");
			SuspendedCount = 1;
		}
		bFirst = false;
	}

	if (bHasCmdLine)
	{
		// Command line takes priority over config
		HangDuration = CmdLine_HangDuration;
		bWalkStackOnHitch = CmdLine_StackWalk;
	}
	else
	{
		static float Config_Duration = -1.0f;
		static bool Config_StackWalk = false;

		// Read from config files
		bool bReadFromConfig = false;
		if (GConfig)
		{
			static bool bDidReadDurationsetting = GConfig->GetFloat(TEXT("Core.System"), TEXT("GameThreadHeartBeatHitchDuration"), Config_Duration, GEngineIni);
			static bool bDidReadStackwalkSetting = GConfig->GetBool(TEXT("Core.System"), TEXT("GameThreadHeartBeatStackWalk"), Config_StackWalk, GEngineIni);
			bReadFromConfig = bDidReadDurationsetting || bDidReadStackwalkSetting;
		}

		if (bReadFromConfig)
		{
			HangDuration = Config_Duration;
			bWalkStackOnHitch = Config_StackWalk;
		}
		else
		{
			// No config provided. Use defaults to disable.
			HangDuration = -1.0f;
			bWalkStackOnHitch = false;
		}
	}

	if (OldHangDuration != HangDuration)
	{
		UE_LOGF(LogCore, Display, "Hitch detector threshold: %dms", int32(HangDuration * 1000.0f));
	}

	// Start the heart beat thread if it hasn't already been started.
	if (Thread == nullptr && (FPlatformProcess::SupportsMultithreading() || FForkProcessHelper::SupportsMultithreadingPostFork()) && HangDuration > 0)
	{
		Thread = FForkProcessHelper::CreateForkableThread(this, TEXT("FGameThreadHitchHeartBeatThreaded"), 0, TPri_AboveNormal);
	}
#endif
}

uint32 FGameThreadHitchHeartBeatThreaded::Run()
{
#if USE_HITCH_DETECTION
#if WALK_STACK_ON_HITCH_DETECTED
	if (bWalkStackOnHitch)
	{
		// Perform a stack trace immediately, so we pay the first time setup cost
		// during engine boot, rather than during game play. The results are discarded.
#if LOOKUP_SYMBOLS_IN_HITCH_STACK_WALK
		FPlatformStackWalk::ThreadStackWalkAndDump(StackTrace, StackTraceSize, 0, GGameThreadId);
#else
		FPlatformStackWalk::CaptureThreadStackBackTrace(GGameThreadId, StackTrace, MaxStackDepth);
#endif
	}
#endif

	while (StopTaskCounter.GetValue() == 0 && !IsEngineExitRequested())
	{
		if (!IsEngineExitRequested() && !GHitchDetected && UE_LOG_ACTIVE(LogCore, Error)) // && !FPlatformMisc::IsDebuggerPresent())
		{
			double LocalFrameStartTime;
			float LocalHangDuration;
			{
				FScopeLock HeartBeatLock(&HeartBeatCritical);
				LocalFrameStartTime = FrameStartTime;
				LocalHangDuration = HangDuration;
			}
			if (LocalFrameStartTime > 0.0 && LocalHangDuration > 0.0f && SuspendedCount == 0)
			{
				const double CurrentTime = Clock.Seconds();
				if (float(CurrentTime - LocalFrameStartTime) > LocalHangDuration)
				{
					if (StopTaskCounter.GetValue() == 0)
					{
						GHitchDetected = true;
						UE_LOGF(LogCore, Error, "Hitch detected on gamethread (frame hasn't finished for %8.2fms):", float(CurrentTime - LocalFrameStartTime) * 1000.0f);
						CSV_EVENT_GLOBAL(TEXT("HitchDetector"));

#if WALK_STACK_ON_HITCH_DETECTED
						if (bWalkStackOnHitch)
						{
							double StartTime = FPlatformTime::Seconds();

#if LOOKUP_SYMBOLS_IN_HITCH_STACK_WALK
							// Walk the stack and dump it to the temp buffer. This process usually allocates a lot of memory.
							StackTrace[0] = 0;
							FPlatformStackWalk::ThreadStackWalkAndDump(StackTrace, StackTraceSize, 0, GGameThreadId);
							FString StackTraceText(StackTrace);
							TArray<FString> StackLines;
							StackTraceText.ParseIntoArrayLines(StackLines);

							UE_LOGF(LogCore, Error, "------Stack start");
							for (FString& StackLine : StackLines)
							{
								UE_LOGF(LogCore, Error, "  %ls", *StackLine);
							}
							UE_LOGF(LogCore, Error, "------Stack end");

#else // LOOKUP_SYMBOLS_IN_HITCH_STACK_WALK == 0

							// Only do a thread stack back trace and print the raw addresses to the log.
							uint32 Depth = FPlatformStackWalk::CaptureThreadStackBackTrace(GGameThreadId, StackTrace, MaxStackDepth);

							UE_LOGF(LogCore, Error, "------Stack start");
							for (uint32 Index = 0; Index < Depth; ++Index)
							{
								UE_LOGF(LogCore, Error, "  0x%016llx", StackTrace[Index]);
							}
							UE_LOGF(LogCore, Error, "------Stack end");

#endif // LOOKUP_SYMBOLS_IN_HITCH_STACK_WALK == 0


							double EndTime = FPlatformTime::Seconds();
							double Duration = EndTime - StartTime;
							UE_LOGF(LogCore, Error, " ## Stack tracing took %f seconds.", Duration);
						}
#endif

						Clock.Tick();
						UE_LOGF(LogCore, Error, "Leaving hitch detector (+%8.2fms)", float(Clock.Seconds() - LocalFrameStartTime) * 1000.0f);
					}
				}
			}
		}
		if (StopTaskCounter.GetValue() == 0 && !IsEngineExitRequested())
		{
			FPlatformProcess::SleepNoStats(0.008f); // check every 8ms
		}

		Clock.Tick();
	}
#endif
	return 0;
}

void FGameThreadHitchHeartBeatThreaded::Stop()
{
	StopTaskCounter.Increment();
}

void FGameThreadHitchHeartBeatThreaded::FrameStart(bool bSkipThisFrame)
{
#if USE_HITCH_DETECTION
	check(IsInGameThread());
	FScopeLock HeartBeatLock(&HeartBeatCritical);
	// Grab this everytime to handle hotfixes
	if (!bSkipThisFrame)
	{
		InitSettingsInternal();
	}
	double Now = Clock.Seconds();
	if (FirstStartTime == 0.0)
	{
		FirstStartTime = Now;
	}
	//if (Now - FirstStartTime > 60.0)
	{
		FrameStartTime = bSkipThisFrame ? 0.0 : Now;
	}
	GHitchDetected = false;
#endif
}

void FGameThreadHitchHeartBeatThreaded::SuspendHeartBeat()
{
#if USE_HITCH_DETECTION
	if (!IsInGameThread())
		return;

	UE_AUTORTFM_OPEN
	{
		FPlatformAtomics::InterlockedIncrement(&SuspendedCount);
		UE_LOGF(LogCore, Log, "HitchHeartBeat Suspend called (count %d) - State: %ls", SuspendedCount, SuspendedCount == 0 ? TEXT("Running") : TEXT("Suspended"));
	};
	UE_AUTORTFM_ONABORT(this)
	{
		ResumeHeartBeat();
	};
#endif
}

void FGameThreadHitchHeartBeatThreaded::ResumeHeartBeat()
{
#if USE_HITCH_DETECTION
	if (!IsInGameThread())
		return;

	// Defer heartbeat-resume to the end of our transaction, if one is running.
	UE_AUTORTFM_ONCOMMIT(this)
	{
		// Temporary workaround for suspend/resume issue
		//check(SuspendedCount > 0);
		if (SuspendedCount == 0)
		{
			UE_LOGF(LogCore, Warning, "HitchHeartBeat Resume called when SuspendedCount was already 0! Ignoring");
			return;
		}

		if (FPlatformAtomics::InterlockedDecrement(&SuspendedCount) == 0)
		{
			FrameStart(true);
		}
		UE_LOGF(LogCore, Log, "HitchHeartBeat Resume called (count %d) - State: %ls", SuspendedCount, SuspendedCount == 0 ? TEXT("Running") : TEXT("Suspended"));
	};
#endif
}

bool FGameThreadHitchHeartBeatThreaded::IsStartedSuspended()
{
#if USE_HITCH_DETECTION
	return bStartSuspended;
#else
	return true;
#endif
}

double FGameThreadHitchHeartBeatThreaded::GetFrameStartTime()
{
#if USE_HITCH_DETECTION
	return FrameStartTime;
#else
	return 0.0;
#endif
}

double FGameThreadHitchHeartBeatThreaded::GetCurrentTime()
{
	return Clock.Seconds();
}
