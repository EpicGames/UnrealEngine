// Copyright Epic Games, Inc. All Rights Reserved.

#include "HAL/ThreadingBase.h"
#include "UObject/NameTypes.h"
#include "Stats/Stats.h"
#include "Misc/CommandLine.h"
#include "Misc/CoreDelegates.h"
#include "Misc/CoreStats.h"
#include "Misc/EventPool.h"
#include "Misc/LazySingleton.h"
#include "Misc/Fork.h"
#include "Misc/Optional.h"
#include "Templates/Atomic.h"
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformStackWalk.h"
#include "ProfilingDebugging/MiscTrace.h"
#include "Async/Fundamental/Scheduler.h"
#include "Tasks/Pipe.h"
#include "AutoRTFM.h"

#if PLATFORM_WINDOWS
#include "Windows/WindowsHWrapper.h"
#endif

#if PLATFORM_MAC
#include "Mac/CocoaThread.h"
#endif

#include <atomic>

#ifndef IS_RUNNING_GAMETHREAD_ON_EXTERNAL_THREAD
#define IS_RUNNING_GAMETHREAD_ON_EXTERNAL_THREAD 0
#endif

DEFINE_STAT( STAT_EventWaitWithId );
DEFINE_STAT( STAT_EventTriggerWithId );

DECLARE_DWORD_COUNTER_STAT( TEXT( "ThreadPoolDummyCounter" ), STAT_ThreadPoolDummyCounter, STATGROUP_ThreadPoolAsyncTasks );

static bool GDoPooledThreadWaitTimeouts = false;
static FAutoConsoleVariableRef CVarDoPooledThreadWaitTimeouts(
	TEXT("DoPooledThreadWaitTimeouts"),
	GDoPooledThreadWaitTimeouts,
	TEXT("If enabled, uses the old behaviour for waking up pool threads every 10ms. Otherwise, lets pooled threads sleep until data arrives."),
	ECVF_Default
);

/** The global thread pool */
FQueuedThreadPool* GThreadPool = nullptr;

FQueuedThreadPool* GIOThreadPool = nullptr;

FQueuedThreadPool* GBackgroundPriorityThreadPool = nullptr;

#if WITH_EDITOR
FQueuedThreadPool* GLargeThreadPool = nullptr;
#endif

int32 FTaskTagScope::GetStaticThreadId()
{
	static int32 ThreadID = FPlatformTLS::GetCurrentThreadId();
	return ThreadID;
}

thread_local ETaskTag FTaskTagScope::ActiveTaskTag = ETaskTag::EStaticInit;
static std::atomic_int ActiveNamedThreads {};

ETaskTag FTaskTagScope::SwapTag(ETaskTag Tag)
{
	ETaskTag ReturnValue = ActiveTaskTag;
	ActiveTaskTag = Tag;
	return ReturnValue;
}

void FTaskTagScope::SetTagNone()
{
	ActiveTaskTag = ETaskTag::ENone;
}

void FTaskTagScope::SetTagStaticInit()
{
	ActiveTaskTag = ETaskTag::EStaticInit;
}

FTaskTagScope::FTaskTagScope(ETaskTag InTag) : Tag(InTag)
{
	checkf(Tag != ETaskTag::ENone, TEXT("None cannot be used as a Tag"));
	checkf(Tag != ETaskTag::EParallelThread, TEXT("Parallel cannot be used on it's own"));

	TagOnlyIfNone = true; // Now defaults to always optional
	if (ActiveTaskTag == ETaskTag::EStaticInit)
	{
		// The static thread ID must be initialized from the first call to this constructor.
		// Otherwise, there is a possibility that it will be initialized on a different thread.
		GetStaticThreadId();

		// Allow overwrite only from EStaticInit -> EGameThread
		TagOnlyIfNone = Tag != ETaskTag::EGameThread;

		checkf(Tag == ETaskTag::EGameThread, TEXT("The Gamethread can only be tagged on the inital thread of the application"));
#if !IS_RUNNING_GAMETHREAD_ON_EXTERNAL_THREAD
#	if PLATFORM_WINDOWS && DO_CHECK
		// When RenderDoc injects its DLL it first creates this process in a suspended
		// state where PE loading is not complete and no static DLL dependencies have
		// been loaded (i.e. no static-init executed). RenderDoc then remotely creates
		// a thread to load renderdoc.dll and it is this thread that then completes
		// the PE loading process prior to calling the thread's entry point. This
		// results static initialisation unexpectedly happening on RenderDoc's injection
		// thread and the ensure below fails.
		static bool bRenderDocDetected = (::GetModuleHandleW(L"renderdoc.dll") != nullptr);
		if (!bRenderDocDetected)
#	endif
		{
			ensureMsgf(IsRunningDuringStaticInit(), TEXT("Static initialization should have happened on the same thread as the main thread"));
		}
#endif
	}

	if (!EnumHasAllFlags(Tag, ETaskTag::EParallelThread))
	{
		ETaskTag NamedThreadBits = (Tag & ETaskTag::ENamedThreadBits);
		static_assert(sizeof(ETaskTag) == sizeof(int32), "EnumSize must match interlockedOr");
		ETaskTag OldTag = ETaskTag(ActiveNamedThreads.fetch_or(int32(NamedThreadBits)));
		bool IsOK = (OldTag & NamedThreadBits) == ETaskTag::ENone;
		if (!IsOK)
		{
			//Try to catch other Threads that already opened a non parallel scope
			ActiveNamedThreads.store(int32(ETaskTag::ENone));
		}
		checkf(IsOK || IsEngineExitRequested(), TEXT("Only Scopes tagged with ETaskTag::EParallelThread can be tagged multiple times. ActiveNamedThreads(%x) cannot be tagged multiple times in the same callstack with the current Tag(%x)"), int32{EnumToUnderlyingType(OldTag)}, int32{EnumToUnderlyingType(Tag)});
	}
	
	ParentTag = ActiveTaskTag;
	if (!TagOnlyIfNone || ActiveTaskTag == ETaskTag::ENone || ActiveTaskTag == ETaskTag::EWorkerThread)
	{
		ActiveTaskTag = Tag;
	}
	else if (TagOnlyIfNone && ActiveTaskTag != Tag)
	{
		if (EnumHasAllFlags(Tag, ETaskTag::EParallelRenderingThread))
		{
			checkf(IsInRenderingThread(), TEXT("ETaskTag::EParallelRenderingThread can only be retagged if they are in a parallel for on the RenderingThread or not tagged check the ActiveNamedThreads(%x)"), int32{EnumToUnderlyingType(FTaskTagScope::GetCurrentTag())});
		}

		if (EnumHasAllFlags(Tag, ETaskTag::EParallelGameThread))
		{
			checkf(IsInGameThread(), TEXT("ETaskTag::EParallelGameThread can only be retagged if they are in a parallel for on the GameThread or not tagged check the ActiveNamedThreads(%x)"), int32{EnumToUnderlyingType(FTaskTagScope::GetCurrentTag())});
		}
	}
}

FTaskTagScope::~FTaskTagScope()
{
	checkf(TagOnlyIfNone || ActiveTaskTag == Tag, TEXT("ActiveTaskTag(%x) corrupted needs to be Tag(%x)"), int32{EnumToUnderlyingType(FTaskTagScope::GetCurrentTag())}, int32{EnumToUnderlyingType(Tag)});
	if (!TagOnlyIfNone || ParentTag == ETaskTag::ENone || ParentTag == ETaskTag::EWorkerThread)
	{
		ActiveTaskTag = ParentTag;
	}

	if (!EnumHasAllFlags(Tag, ETaskTag::EParallelThread))
	{
		ETaskTag NamedThreadBits = (Tag & ETaskTag::ENamedThreadBits);
		static_assert(sizeof(ETaskTag) == sizeof(int32), "EnumSize must match interlockedAnd");
		ETaskTag OldTag = ETaskTag(ActiveNamedThreads.fetch_and(int32(~NamedThreadBits)));
		checkf((OldTag & NamedThreadBits) == (NamedThreadBits) || IsEngineExitRequested(), TEXT("Currently active Threads(%x) got corrupted check the ActiveNamedThreads(%x)"), int32{EnumToUnderlyingType(OldTag)}, int32{EnumToUnderlyingType(FTaskTagScope::GetCurrentTag())});
	}

	//prolong the scope of the GT for static variable destructors
	if (Tag == ETaskTag::EGameThread && ActiveTaskTag == ETaskTag::EStaticInit)
	{
		ActiveTaskTag = ETaskTag::EGameThread;
	}
}

bool FTaskTagScope::IsCurrentTag(ETaskTag InTag)
{
	return ActiveTaskTag == InTag;
}

ETaskTag FTaskTagScope::GetCurrentTag()
{
	return ActiveTaskTag;
}

bool FTaskTagScope::IsRunningDuringStaticInit()
{
	return ActiveTaskTag == ETaskTag::EStaticInit && GetStaticThreadId() == FPlatformTLS::GetCurrentThreadId();
}

CORE_API bool IsInGameThread()
{
	if (GIsGameThreadIdInitialized)
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		bool newValue = FTaskTagScope::IsCurrentTag(ETaskTag::EGameThread) || FTaskTagScope::IsRunningDuringStaticInit();
#if !UE_BUILD_SHIPPING && !UE_BUILD_TEST && !USING_INSTRUMENTATION
		if (!UE::Tasks::Private::IsThreadRetractingTask())
		{
			const uint32 CurrentThreadId = FPlatformTLS::GetCurrentThreadId();
			bool oldValue = CurrentThreadId == GGameThreadId;
			ensureMsgf(oldValue == newValue, TEXT("oldValue(%i) newValue(%i) If this check fails make sure that there is a FTaskTagScope(ETaskTag::EGameThread) as deep as possible on the current callstack, you can see the current value in ActiveNamedThreads(%x)"), oldValue, newValue, int32{EnumToUnderlyingType(FTaskTagScope::GetCurrentTag())});
			newValue = oldValue;
		}
#endif
		return newValue;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	return true;
}

CORE_API bool IsInParallelGameThread()
{
	return FTaskTagScope::IsCurrentTag(ETaskTag::EParallelGameThread);
}

CORE_API bool IsInParallelLoadingThread()
{
	return FTaskTagScope::IsCurrentTag(ETaskTag::EParallelLoadingThread);
}

CORE_API bool IsInWorkerThread()
{
	return LowLevelTasks::FScheduler::Get().IsWorkerThread();
}

CORE_API bool IsInSlateThread()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// If this explicitly is a slate thread, not just the main thread running slate
	bool newValue = FTaskTagScope::IsCurrentTag(ETaskTag::ESlateThread);
#if !UE_BUILD_SHIPPING && !UE_BUILD_TEST && !USING_INSTRUMENTATION
	if (!UE::Tasks::Private::IsThreadRetractingTask())
	{
		bool oldValue = GSlateLoadingThreadId != 0 && FPlatformTLS::GetCurrentThreadId() == GSlateLoadingThreadId;
		ensureMsgf(oldValue == newValue, TEXT("oldValue(%i) newValue(%i) If this check fails make sure that there is a FTaskTagScope(ETaskTag::ESlateThread) as as deep as possible on the current callstack, you can see the current value in ActiveNamedThreads(%x)"), oldValue, newValue, int32{EnumToUnderlyingType(FTaskTagScope::GetCurrentTag())});
		newValue = oldValue;
	}
#endif
	return newValue;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

// tasks pipe that arranges audio tasks execution one after another so no synchronisation between them is required
CORE_API UE::Tasks::FPipe GAudioPipe{ TEXT("AudioPipe") };
// True if async audio processing is enabled and started
CORE_API std::atomic<bool> GIsAudioThreadRunning{ false };

CORE_API std::atomic<bool> GIsAudioThreadSuspended{ false };

CORE_API bool IsAudioThreadRunning()
{
	return GIsAudioThreadRunning.load(std::memory_order_acquire) && !GIsAudioThreadSuspended.load(std::memory_order_acquire);
}

CORE_API bool IsInAudioThread()
{
	return (GIsAudioThreadRunning.load(std::memory_order_acquire) && !GIsAudioThreadSuspended.load(std::memory_order_acquire)) ? GAudioPipe.IsInContext() : IsInGameThread();
}

CORE_API bool IsInActualRenderingThread()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	bool newValue = FTaskTagScope::IsCurrentTag(ETaskTag::ERenderingThread);
#if !UE_BUILD_SHIPPING && !UE_BUILD_TEST
	{
		bool oldValue = FPlatformTLS::GetCurrentThreadId() == GRenderThreadId;
		ensureMsgf(oldValue == newValue, TEXT("oldValue(%i) newValue(%i) If this check fails make sure that there is a FTaskTagScope(ETaskTag::ERenderingThread) as deep as possible on the current callstack, you can see the current value in ActiveNamedThreads(%x)"), oldValue, newValue, int32{EnumToUnderlyingType(FTaskTagScope::GetCurrentTag())});
		newValue = oldValue;
	}
#endif
	return newValue;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

CORE_API bool IsInRenderingThread()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	bool newValue = (GRenderThreadId == 0)
		? FTaskTagScope::IsCurrentTag(ETaskTag::EGameThread) || FTaskTagScope::IsCurrentTag(ETaskTag::ERenderingThread) || FTaskTagScope::IsRunningDuringStaticInit()
		: FTaskTagScope::IsCurrentTag(ETaskTag::ERenderingThread);

#if !UE_BUILD_SHIPPING && !UE_BUILD_TEST && !USING_INSTRUMENTATION
	if (!UE::Tasks::Private::IsThreadRetractingTask())
	{
		const uint32 CurrentThreadId = FPlatformTLS::GetCurrentThreadId();
		bool oldValue = (GRenderThreadId == 0) ? (CurrentThreadId == GGameThreadId) || FTaskTagScope::IsRunningDuringStaticInit() : (CurrentThreadId == GRenderThreadId);
		ensureMsgf(oldValue == newValue, TEXT("oldValue(%i) newValue(%i) If this check fails make sure that there is a FTaskTagScope(ETaskTag::ERenderingThread) as deep as possible on the current callstack, you can see the current value in ActiveNamedThreads(%x)"), oldValue, newValue, int32{EnumToUnderlyingType(FTaskTagScope::GetCurrentTag())});
		newValue = oldValue;
	}
#endif
	return newValue;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

CORE_API bool IsInAnyRenderingThread()
{
	return FTaskTagScope::IsCurrentTag(ETaskTag::EParallelRenderingThread)
		|| FTaskTagScope::IsCurrentTag(ETaskTag::ERenderingThread)
		|| FTaskTagScope::IsCurrentTag(ETaskTag::EParallelRhiThread)
		|| FTaskTagScope::IsCurrentTag(ETaskTag::ERhiThread);
}

CORE_API bool IsInParallelRenderingThread()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	bool newValue = false;
	if (GRenderThreadId == 0)
	{
		newValue = FTaskTagScope::IsCurrentTag(ETaskTag::ERenderingThread) || FTaskTagScope::IsCurrentTag(ETaskTag::EGameThread) || FTaskTagScope::IsCurrentTag(ETaskTag::EParallelRenderingThread);
	}
	else
	{
		newValue = FTaskTagScope::IsCurrentTag(ETaskTag::EParallelRenderingThread)
			|| FTaskTagScope::IsCurrentTag(ETaskTag::ERenderingThread)
			|| FTaskTagScope::IsCurrentTag(ETaskTag::EParallelRhiThread) //TODO lots of RHI functions rely on our broken IsInParallelRenderingThread;
			|| FTaskTagScope::IsCurrentTag(ETaskTag::ERhiThread); //TODO lots of RHI functions rely on our broken IsInParallelRenderingThread;
	}

#if !UE_BUILD_SHIPPING && !UE_BUILD_TEST && !USING_INSTRUMENTATION
	if (!UE::Tasks::Private::IsThreadRetractingTask())
	{
		const uint32 CurrentThreadId = FPlatformTLS::GetCurrentThreadId();
		bool oldValue = (GRenderThreadId == 0) ?  true : CurrentThreadId != GGameThreadId;
		ensureMsgf(oldValue == newValue, TEXT("oldValue(%i) newValue(%i) If this check fails make sure that there is a FTaskTagScope(ETaskTag::EParallelRenderingThread) as deep as possible on the current callstack, you can see the current value in ActiveNamedThreads(%x)"), oldValue, newValue, int32{EnumToUnderlyingType(FTaskTagScope::GetCurrentTag())});
		newValue = oldValue;
	}
#endif
	return newValue;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

CORE_API bool GIsRunningRHIInSeparateThread_InternalUseOnly  = false;
CORE_API bool GIsRunningRHIInDedicatedThread_InternalUseOnly = false;
CORE_API bool GIsRunningRHIInTaskThread_InternalUseOnly      = false;

CORE_API bool IsRHIThreadRunning()
{
	return GIsRunningRHIInDedicatedThread_InternalUseOnly;
}

CORE_API bool IsInRHIThread()
{
	return FTaskTagScope::IsCurrentTag(ETaskTag::ERhiThread);
}

CORE_API bool IsInParallelRHIThread()
{
	return (FTaskTagScope::GetCurrentTag() & ETaskTag::ERhiThread) == ETaskTag::ERhiThread;
}

// Fake threads

// Core version of IsInAsyncLoadingThread
static bool IsInAsyncLoadingThreadCoreInternal()
{
	// No async loading in Core
	return false;
}
bool(*IsInAsyncLoadingThread)() = &IsInAsyncLoadingThreadCoreInternal;

CORE_API bool IsInActualLoadingThread()
{
	return IsInAsyncLoadingThread() && !IsInGameThread();
}

/**
 * Fake thread created when multi-threading is disabled.
 */
class FFakeThread : public FRunnableThread
{
	/** Thread Id pool */
	static uint32 ThreadIdCounter;

protected:

	/** Thread is suspended. */
	bool bIsSuspended;

	/** Runnable object associated with this thread. */
	FSingleThreadRunnable* SingleThreadRunnable;

public:

	/** Use the MSB as a mask to prevent clashes between kernel assigned thread ids and fake thread ids */
	static constexpr uint32 FakeIdReservedBit = 1 << 31;

	/** Constructor. */
	FFakeThread()
		: bIsSuspended(false)
		, SingleThreadRunnable(nullptr)
	{
		ThreadID = ThreadIdCounter++;
		ThreadID |= FakeIdReservedBit;
		// Auto register with single thread manager.
		FThreadManager::Get().AddThread(ThreadID, this);
	}

	/** Virtual destructor. */
	virtual ~FFakeThread()
	{
		// Remove from the manager.
		FThreadManager::Get().RemoveThread(this);
	}

	/** Tick one time per frame. */
	virtual void Tick() override
	{
		if (SingleThreadRunnable && !bIsSuspended)
		{
			SingleThreadRunnable->Tick();
		}
	}

public:

	// FRunnableThread interface

	virtual void SetThreadPriority(EThreadPriority NewPriority) override
	{
		// Not relevant.
	}

	virtual void Suspend(bool bShouldPause) override
	{
		bIsSuspended = bShouldPause;
	}

	virtual bool Kill(bool bShouldWait) override
	{
		FThreadManager::Get().RemoveThread(this);
		return true;
	}

	virtual void WaitForCompletion() override
	{
		FThreadManager::Get().RemoveThread(this);
	}

	virtual FRunnableThread::ThreadType GetThreadType() const override
	{
		return ThreadType::Fake;
	}

	virtual bool CreateInternal(FRunnable* InRunnable, const TCHAR* InThreadName,
		uint32 InStackSize,
		EThreadPriority InThreadPri, uint64 InThreadAffinityMask,
		EThreadCreateFlags InCreateFlags = EThreadCreateFlags::None) override

	{
		ThreadName = InThreadName;
		ThreadAffinityMask = InThreadAffinityMask;

		SingleThreadRunnable = InRunnable->GetSingleThreadInterface();
		if (SingleThreadRunnable)
		{
			InRunnable->Init();

			Runnable = InRunnable;
		}
		return SingleThreadRunnable != nullptr;
	}
};
uint32 FFakeThread::ThreadIdCounter = 0xffff;

FThreadManager::FThreadManager()
{
	FCoreDelegates::OnParentPreFork.AddRaw(this, &FThreadManager::HandleOnParentPreFork);
}

FThreadManager::~FThreadManager()
{
	FCoreDelegates::OnParentPreFork.RemoveAll(this);
}

bool FThreadManager::CheckThreadListSafeToContinueIteration()
{
	if (bIsThreadListDirty)
	{
		UE_LOGF(LogCore, Error, "FThreadManager::Threads was modified during unsafe iteration. Iteration will be aborted.");
		return false;
	}

	return true;
}

void FThreadManager::OnThreadListModified()
{
	bIsThreadListDirty = true;
}

void FThreadManager::AddThread(uint32 ThreadId, FRunnableThread* Thread)
{
	// Convert the thread's priority into an ordered value that is suitable
	// for sorting. Note we're using higher values so as to not collide with
	// existing trace data that's using TPri directly, and leaving gaps so
	// values can be added in between should need be
	int8 PriRemap[][2] = {
		{ TPri_TimeCritical,		0x10 },
		{ TPri_Highest,				0x20 },
		{ TPri_AboveNormal,			0x30 },
		{ TPri_Normal,				0x40 },
		{ TPri_SlightlyBelowNormal,	0x50 },
		{ TPri_BelowNormal,			0x60 },
		{ TPri_Lowest,				0x70 },
	};
	static_assert(TPri_Num == UE_ARRAY_COUNT(PriRemap), "Please update PriRemap when adding/removing thread priorities. Many thanks.");
	int32 SortHint = UE_ARRAY_COUNT(PriRemap);
	for (auto Candidate : PriRemap)
	{
		if (Candidate[0] == Thread->GetThreadPriority())
		{
			SortHint = Candidate[1];
			break;
		}
	}

	// Note that this must be called from thread being registered.
	UE::Trace::ThreadRegister(*(Thread->GetThreadName()), Thread->GetThreadID(), SortHint);

	const bool bIsSingleThreadEnvironment = FPlatformProcess::SupportsMultithreading() == false;

	if (bIsSingleThreadEnvironment && Thread->GetThreadType() == FRunnableThread::ThreadType::Real)
	{
		checkf((ThreadId & FFakeThread::FakeIdReservedBit) == 0, TEXT("The thread ID  assigned by the kernel clashes with the bit reserved for identifying fake threads. Need to revisit the fake ID assignment algo."));
	}

	bool bAlreadyInThreadPool = false;
	{
		const FScopeLock ThreadsLock(&ThreadsCritical);
		// Some platforms do not support TLS
		bAlreadyInThreadPool = Threads.Contains(ThreadId);
		if (!bAlreadyInThreadPool)
		{
			Threads.Add(ThreadId, Thread);
			OnThreadListModified();
		}
	}
	// notify observers only about new threads
	if (!bAlreadyInThreadPool)
	{
		ThreadEvent.Broadcast(ThreadId, Thread, EThreadEventType::Added);
	}
}

void FThreadManager::RemoveThread(FRunnableThread* Thread)
{
	TOptional<uint32> RemovedThreadId;
	{
		FScopeLock ThreadsLock(&ThreadsCritical);
		const uint32* ThreadId = Threads.FindKey(Thread);
		if (ThreadId)
		{
			RemovedThreadId.Emplace(*ThreadId);
			// `ThreadId` can be invalidated after it
			Threads.Remove(*ThreadId);
			OnThreadListModified();
		}
	}
	// notify observers only about existing threads
	if (RemovedThreadId)
	{
		ThreadEvent.Broadcast(RemovedThreadId.GetValue(), Thread, EThreadEventType::Removed);
	}
}

FThreadManager::FThreadEvent& FThreadManager::OnThreadEvent()
{
	return ThreadEvent;
}

void FThreadManager::Tick()
{	
	const bool bIsSingleThreadEnvironment = FPlatformProcess::SupportsMultithreading() == false;
	if (bIsSingleThreadEnvironment)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FSingleThreadManager_Tick);

		ForEachThread(
			[] (uint32 ThreadId, FRunnableThread* Thread)
			{
				// Only fake and forkable threads are ticked by the ThreadManager
				if (Thread->GetThreadType() != FRunnableThread::ThreadType::Real)
				{
					Thread->Tick();
				}
			}
		);
	}
}

const FString& FThreadManager::GetThreadNameInternal(uint32 ThreadId)
{
	static FString NoThreadName;
	FScopeLock ThreadsLock(&ThreadsCritical);
	FRunnableThread** Thread = Threads.Find(ThreadId);
	if (Thread)
	{
		return (*Thread)->GetThreadName();
	}
	return NoThreadName;
}

#if PLATFORM_SUPPORTS_ALL_THREAD_BACKTRACES
static TConstArrayView<uint64> ThreadStackBackTraces_PerformStackWalk(uint32 CurThreadId, uint32 ThreadId, TArrayView<uint64> OutProgramCounters)
{
	uint32 Depth;
	if (CurThreadId != ThreadId)
	{
		Depth = FPlatformStackWalk::CaptureThreadStackBackTrace(ThreadId, OutProgramCounters.GetData(), OutProgramCounters.Num());
	}
	else
	{
		Depth = FPlatformStackWalk::CaptureStackBackTrace(OutProgramCounters.GetData(), OutProgramCounters.Num());
	}
	return OutProgramCounters.Left(Depth);
}

static void GetAllThreadStackBackTraces_ProcessSingle(
	uint32 CurThreadId,
	uint32 ThreadId,
	const TCHAR* ThreadName,
	typename FThreadManager::FThreadStackBackTrace& OutStackTrace)
{
	OutStackTrace.ThreadId = ThreadId;
	OutStackTrace.ThreadName = ThreadName;

	FThreadManager::FThreadStackBackTrace::FProgramCountersArray& PCs = OutStackTrace.ProgramCounters;
	PCs.SetNumZeroed(FThreadManager::FThreadStackBackTrace::ProgramCountersMaxStackSize);
	const TConstArrayView<uint64> WrittenProgramCounters = ThreadStackBackTraces_PerformStackWalk(CurThreadId, ThreadId, PCs);
	PCs.SetNum(WrittenProgramCounters.Num());
}

void FThreadManager::GetAllThreadStackBackTraces(TArray<FThreadStackBackTrace>& StackTraces)
{
	const uint32 CurThreadId = FPlatformTLS::GetCurrentThreadId();
	FScopeLock Lock(&ThreadsCritical);
	const int32 NumThreads = Threads.Num() + 1;

	StackTraces.Empty(NumThreads);
#if PLATFORM_MAC && MAC_SEPARATE_GAME_THREAD
	GetAllThreadStackBackTraces_ProcessSingle(CurThreadId, GMacMainThreadId, TEXT("MacMainThread"), StackTraces.AddDefaulted_GetRef());
#endif
	GetAllThreadStackBackTraces_ProcessSingle(CurThreadId, GGameThreadId, TEXT("GameThread"), StackTraces.AddDefaulted_GetRef());

	ForEachThread(
		[CurThreadId, &StackTraces] (uint32 ThreadId, FRunnableThread* Thread)
		{
			const FString& Name = Thread->GetThreadName();
			GetAllThreadStackBackTraces_ProcessSingle(CurThreadId, ThreadId, *Name, StackTraces.AddDefaulted_GetRef());
		}
	);
}

void FThreadManager::ForEachThreadStackBackTrace(TFunctionRef<bool(uint32 ThreadId, const TCHAR* ThreadName, const TConstArrayView<uint64>& StackTrace)> Func)
{
	const uint32 CurThreadId = FPlatformTLS::GetCurrentThreadId();
	FThreadStackBackTrace::FProgramCountersArray ProgramCounterBuffer;
	ProgramCounterBuffer.SetNumZeroed(FThreadStackBackTrace::ProgramCountersMaxStackSize);

	FScopeLock Lock(&ThreadsCritical);
	bIsThreadListDirty = false;
	
#if PLATFORM_MAC && MAC_SEPARATE_GAME_THREAD
	{
		const TConstArrayView<uint64> ProgramCounterOutputArrayView = ThreadStackBackTraces_PerformStackWalk(CurThreadId, GMacMainThreadId, ProgramCounterBuffer);
		const bool bContinue = Func(GMacMainThreadId, TEXT("MacMainThread"), ProgramCounterOutputArrayView);
		if (!CheckThreadListSafeToContinueIteration() || !bContinue)
		{
			return;
		}
	}
#endif

	{
		const TConstArrayView<uint64> ProgramCounterOutputArrayView = ThreadStackBackTraces_PerformStackWalk(CurThreadId, GGameThreadId, ProgramCounterBuffer);
		const bool bContinue = Func(GGameThreadId, TEXT("GameThread"), ProgramCounterOutputArrayView);
		if (!CheckThreadListSafeToContinueIteration() || !bContinue)
		{
			return;
		}
	}

	for (const TPair<uint32, FRunnableThread*>& Pair : Threads)
	{
		const uint32 ThreadId = Pair.Key;
		const FRunnableThread* Thread = Pair.Value;
		const FString& ThreadName = Thread->GetThreadName();
		const TConstArrayView<uint64> ProgramCounterOutputArrayView = ThreadStackBackTraces_PerformStackWalk(CurThreadId, ThreadId, ProgramCounterBuffer);
		const bool bContinue = Func(ThreadId, *ThreadName, ProgramCounterOutputArrayView);
		if (!CheckThreadListSafeToContinueIteration() || !bContinue)
		{
			return;
		}
	}
}
#endif

void FThreadManager::ForEachThread(TFunction<void(uint32, FRunnableThread*)> Func)
{
	FScopeLock Lock(&ThreadsCritical);
	// threads can be added or removed while iterating over them, thus invalidating the iterator, so we iterate over the copy of threads collection
	FThreads ThreadsCopy = Threads;

	for (const TPair<uint32, FRunnableThread*>& Pair : ThreadsCopy)
	{
		Func(Pair.Key, Pair.Value);
	}
}

FThreadManager& FThreadManager::Get()
{
	static FThreadManager Singleton;
	return Singleton;
}

TArray<FRunnableThread*> FThreadManager::GetForkableThreads()
{
	TArray<FRunnableThread*> ForkableThreads;
	ForEachThread(
		[&ForkableThreads] (uint32 ThreadId, FRunnableThread* Thread)
		{
			if (Thread->GetThreadType() == FRunnableThread::ThreadType::Forkable)
			{
				ForkableThreads.Add(Thread);
			}
		}
	);

	return ForkableThreads;
}

void FThreadManager::HandleOnParentPreFork()
{
	// Use a local list of forkable threads so we don't keep a lock on the global list during thread creation
	TArray<FRunnableThread*> ForkableThreads = GetForkableThreads();
	for (FRunnableThread* ForkableThread : ForkableThreads)
	{
		ForkableThread->OnPreFork();
	}
}

/*-----------------------------------------------------------------------------
	FEvent, FScopedEvent
-----------------------------------------------------------------------------*/

TAtomic<uint32> FEvent::EventUniqueId;

void FEvent::AdvanceStats()
{
#if	STATS
	EventId = EventUniqueId++;
	EventStartCycles = 0;
#endif // STATS
}

void FEvent::WaitForStats()
{
#if	STATS
	// Only start counting on the first wait, trigger will "close" the history.
	if( FThreadStats::IsCollectingData() && EventStartCycles.Load(EMemoryOrder::Relaxed) == 0 )
	{
		const uint64 PacketEventIdAndCycles = ((uint64)EventId << 32) | 0;
		STAT_ADD_CUSTOMMESSAGE_PTR( STAT_EventWaitWithId, PacketEventIdAndCycles );
		EventStartCycles = FPlatformTime::Cycles();
	}
#endif // STATS
}

void FEvent::TriggerForStats()
{
#if	STATS
	// Only add wait-trigger pairs.
	uint32 LocalEventStartCycles = EventStartCycles.Load(EMemoryOrder::Relaxed);
	if( LocalEventStartCycles > 0 && FThreadStats::IsCollectingData() )
	{
		const uint32 EndCycles = FPlatformTime::Cycles();
		const int32 DeltaCycles = int32( EndCycles - LocalEventStartCycles );
		const uint64 PacketEventIdAndCycles = ((uint64)EventId << 32) | DeltaCycles;
		STAT_ADD_CUSTOMMESSAGE_PTR( STAT_EventTriggerWithId, PacketEventIdAndCycles );

		AdvanceStats();
	}
#endif // STATS
}

void FEvent::ResetForStats()
{
#if	STATS
	AdvanceStats();
#endif // STATS
}

FScopedEvent::FScopedEvent()
	: Event(TLazySingleton<TEventPool<EEventMode::AutoReset>>::Get().GetRawEvent())
{ }

bool FScopedEvent::IsReady()
{
	if ( Event && Event->Wait(1) )
	{
		TLazySingleton<TEventPool<EEventMode::AutoReset>>::Get().ReturnRawEvent(Event);
		Event = nullptr;
		return true;
	}
	return Event == nullptr;
}

FScopedEvent::~FScopedEvent()
{
	if(Event)
	{
		Event->Wait();
		TLazySingleton<TEventPool<EEventMode::AutoReset>>::Get().ReturnRawEvent(Event);
	}
}


/*-----------------------------------------------------------------------------
	FEventRef
-----------------------------------------------------------------------------*/

FEventRef::FEventRef(EEventMode Mode /* = EEventMode::AutoReset */)
{
	if (Mode == EEventMode::AutoReset)
	{
		Event = TLazySingleton<TEventPool<EEventMode::AutoReset>>::Get().GetRawEvent();
	}
	else
	{
		Event = TLazySingleton<TEventPool<EEventMode::ManualReset>>::Get().GetRawEvent();
	}
}

FEventRef::~FEventRef()
{
	if (Event->IsManualReset())
	{
		TLazySingleton<TEventPool<EEventMode::ManualReset>>::Get().ReturnRawEvent(Event);
	}
	else
	{
		TLazySingleton<TEventPool<EEventMode::AutoReset>>::Get().ReturnRawEvent(Event);
	}
}

/*-----------------------------------------------------------------------------
	FEventPtr
-----------------------------------------------------------------------------*/

FSharedEventRef::FSharedEventRef(EEventMode Mode  /* = EEventMode::AutoReset */)
	: Ptr(Mode == EEventMode::ManualReset
			? TLazySingleton<TEventPool<EEventMode::ManualReset>>::Get().GetRawEvent()
			: TLazySingleton<TEventPool<EEventMode::AutoReset>>::Get().GetRawEvent(),
		[](FEvent* Event)
		{
			if (Event->IsManualReset())
			{
				TLazySingleton<TEventPool<EEventMode::ManualReset>>::Get().ReturnRawEvent(Event);
			}
			else
			{
				TLazySingleton<TEventPool<EEventMode::AutoReset>>::Get().ReturnRawEvent(Event);
			}
		})
{
}

/*-----------------------------------------------------------------------------
	FRunnableThread
-----------------------------------------------------------------------------*/

uint32 FRunnableThread::RunnableTlsSlot = FRunnableThread::GetTlsSlot();

uint32 FRunnableThread::GetTlsSlot()
{
	check( IsInGameThread() );
	uint32 TlsSlot = FPlatformTLS::AllocTlsSlot();
	check( FPlatformTLS::IsValidTlsSlot( TlsSlot ) );
	return TlsSlot;
}

FRunnableThread::FRunnableThread()
	: Runnable(nullptr)
	, ThreadInitSyncEvent(nullptr)
	, ThreadAffinityMask(FPlatformAffinity::GetNoAffinityMask())
	, ThreadPriority(TPri_Normal)
	, ThreadID(0)
{
}

FRunnableThread::~FRunnableThread()
{
	if (!IsEngineExitRequested())
	{
		FThreadManager::Get().RemoveThread(this);
	}
}

FRunnableThread* FRunnableThread::Create(
	class FRunnable* InRunnable, 
	const TCHAR* ThreadName,
	uint32 InStackSize,
	EThreadPriority InThreadPri, 
	uint64 InThreadAffinityMask,
	EThreadCreateFlags InCreateFlags)
{
	bool bCreateRealThread = FPlatformProcess::SupportsMultithreading();

	FRunnableThread* NewThread = nullptr;

	if (bCreateRealThread)
	{
		check(InRunnable);
		// Create a new thread object
		NewThread = FPlatformProcess::CreateRunnableThread();
	}
	else if (InRunnable->GetSingleThreadInterface())
	{
		// Create a fake thread when multithreading is disabled.
		NewThread = new FFakeThread();
	}

	if (NewThread)
	{
		SetupCreatedThread(NewThread, InRunnable, ThreadName, InStackSize, InThreadPri, InThreadAffinityMask, InCreateFlags);
	}

	return NewThread;
}

void FRunnableThread::SetupCreatedThread(FRunnableThread*& NewThread, class FRunnable* InRunnable,  const TCHAR* ThreadName, uint32 InStackSize, EThreadPriority InThreadPri, uint64 InThreadAffinityMask, EThreadCreateFlags InCreateFlags)
{
	// Call the thread's create method
	bool bIsValid = NewThread->CreateInternal(InRunnable, ThreadName, InStackSize, InThreadPri, InThreadAffinityMask, InCreateFlags);

	if( bIsValid )
	{
		check(NewThread->Runnable);
		NewThread->PostCreate(InThreadPri);
	}
	else
	{
		// We failed to start the thread correctly so clean up
		delete NewThread;
		NewThread = nullptr;
	}
}

void FRunnableThread::PostCreate(EThreadPriority InThreadPriority)
{
#if	STATS
	AutoRTFM::UnreachableIfClosed("TODO: Can remove this if we mark `FRunnableThread` as `AUTORTFM_DISABLE`!");
	FStartupMessages::Get().AddThreadMetadata( FName( *GetThreadName() ), GetThreadID() );
#endif // STATS
}

void FRunnableThread::SetTls()
{
	// Make sure it's called from the owning thread.
	check( ThreadID == FPlatformTLS::GetCurrentThreadId() );
	check( FPlatformTLS::IsValidTlsSlot(RunnableTlsSlot) );
	FPlatformTLS::SetTlsValue( RunnableTlsSlot, this );
	FTaskTagScope::SetTagNone();
}

void FRunnableThread::FreeTls()
{
	// Make sure it's called from the owning thread.
	check( ThreadID == FPlatformTLS::GetCurrentThreadId() );
	check( FPlatformTLS::IsValidTlsSlot(RunnableTlsSlot) );
	FPlatformTLS::SetTlsValue( RunnableTlsSlot, nullptr );
}

/*-----------------------------------------------------------------------------
	FThreadPoolPriorityQueue
-----------------------------------------------------------------------------*/

FThreadPoolPriorityQueue::FThreadPoolPriorityQueue()
	: NumQueuedWork(0)
{
}

void FThreadPoolPriorityQueue::Enqueue(IQueuedWork* InQueuedWork, EQueuedWorkPriority InPriority)
{
	int32 QueueIndex = static_cast<int32>(InPriority);
	if (PriorityQueuedWork.Num() <= QueueIndex)
	{
		PriorityQueuedWork.SetNum(QueueIndex + 1);
	}

	NumQueuedWork++;
	if (QueueIndex < FirstNonEmptyQueueIndex)
	{
		FirstNonEmptyQueueIndex = QueueIndex;
	}
	PriorityQueuedWork[QueueIndex].Add(InQueuedWork);
}

bool FThreadPoolPriorityQueue::Retract(IQueuedWork* InQueuedWork)
{
	for (int32 QueueIndex = FirstNonEmptyQueueIndex, Num = PriorityQueuedWork.Num(); QueueIndex < Num; ++QueueIndex)
	{
		if (PriorityQueuedWork[QueueIndex].RemoveSingle(InQueuedWork))
		{
			NumQueuedWork--;
			return true;
		}
	}

	return false;
}

IQueuedWork* FThreadPoolPriorityQueue::Dequeue(EQueuedWorkPriority* OutDequeuedWorkPriority)
{
	IQueuedWork* Work = nullptr;
	for (int32 QueueIndex = FirstNonEmptyQueueIndex, Num = PriorityQueuedWork.Num(); QueueIndex < Num; ++QueueIndex)
	{
		TArray<IQueuedWork*>& QueuedWork = PriorityQueuedWork[QueueIndex];
		if (QueuedWork.Num() > 0)
		{
			// Grab the oldest work in the queue. This is slower than
			// getting the most recent but prevents work from being
			// queued and never done
			Work = QueuedWork[0];
			// Remove it from the list so no one else grabs it
			QueuedWork.RemoveAt(0, EAllowShrinking::No);

			FirstNonEmptyQueueIndex = QueueIndex;
			NumQueuedWork--;

			if (OutDequeuedWorkPriority)
			{
				*OutDequeuedWorkPriority = (EQueuedWorkPriority)QueueIndex;
			}

			break;
		}
	}

	return Work;
}

IQueuedWork* FThreadPoolPriorityQueue::Peek(EQueuedWorkPriority* OutDequeuedWorkPriority) const
{
	IQueuedWork* Work = nullptr;
	for (int32 QueueIndex = FirstNonEmptyQueueIndex, Num = PriorityQueuedWork.Num(); QueueIndex < Num; ++QueueIndex)
	{
		const TArray<IQueuedWork*>& QueuedWork = PriorityQueuedWork[QueueIndex];
		if (QueuedWork.Num() > 0)
		{
			Work = QueuedWork[0];

			if (OutDequeuedWorkPriority)
			{
				*OutDequeuedWorkPriority = (EQueuedWorkPriority)QueueIndex;
			}

			break;
		}
	}

	return Work;
}

void FThreadPoolPriorityQueue::Reset()
{
	PriorityQueuedWork.Empty();
	FirstNonEmptyQueueIndex = 0;
	NumQueuedWork = 0;
}

void FThreadPoolPriorityQueue::Sort(EQueuedWorkPriority InPriorityBucket, TFunctionRef<bool(const IQueuedWork* A, const IQueuedWork* B)> Predicate)
{
	int32 QueueIndex = static_cast<int32>(InPriorityBucket);
	if (QueueIndex < PriorityQueuedWork.Num())
	{
		Algo::Sort(PriorityQueuedWork[QueueIndex], Predicate);
	}
}

/*-----------------------------------------------------------------------------
	FQueuedThread
-----------------------------------------------------------------------------*/

/**
 * This is the interface used for all poolable threads. The usage pattern for
 * a poolable thread is different from a regular thread and this interface
 * reflects that. Queued threads spend most of their life cycle idle, waiting
 * for work to do. When signaled they perform a job and then return themselves
 * to their owning pool via a callback and go back to an idle state.
 */
class FQueuedThread
	: public FRunnable
{
protected:

	/** The event that tells the thread there is work to do. */
	FEvent* DoWorkEvent = nullptr;

	/** If true, the thread should exit. */
	TAtomic<bool> TimeToDie { false };

	/** The work this thread is doing. */
	IQueuedWork* volatile QueuedWork = nullptr;

	/** The pool this thread belongs to. */
	class FQueuedThreadPoolBase* OwningThreadPool = nullptr;

	/** My Thread  */
	FRunnableThread* Thread = nullptr;

	/**
	 * The real thread entry point. It waits for work events to be queued. Once
	 * an event is queued, it executes it and goes back to waiting.
	 */
	virtual uint32 Run() override;

	/** Common code use by Create functions */
	FString SetupForCreate(class FQueuedThreadPoolBase* InPool, const TCHAR* ThreadName)
	{
		static int32 PoolThreadIndex = 0;
		const FString PoolThreadName = ThreadName ? FString(ThreadName) : FString::Printf(TEXT("PoolThread %d"), PoolThreadIndex++);

		OwningThreadPool = InPool;
		DoWorkEvent = FPlatformProcess::GetSynchEventFromPool();
		return PoolThreadName;
	}

public:

	/** Default constructor **/
	FQueuedThread() = default;

	/**
	 * Creates the thread with the specified stack size and creates the various
	 * events to be able to communicate with it.
	 *
	 * @param InPool The thread pool interface used to place this thread back into the pool of available threads when its work is done
	 * @param InStackSize The size of the stack to create. 0 means use the current thread's stack size
	 * @param ThreadPriority priority of new thread
	 * @return True if the thread and all of its initialization was successful, false otherwise
	 */
	virtual bool Create(class FQueuedThreadPoolBase* InPool,uint32 InStackSize = 0, EThreadPriority ThreadPriority=TPri_Normal, const TCHAR* ThreadName = nullptr)
	{
		const FString PoolThreadName = SetupForCreate(InPool, ThreadName);

		Thread = FRunnableThread::Create(this, *PoolThreadName, InStackSize, ThreadPriority, FPlatformAffinity::GetPoolThreadMask());
		check(Thread);
		return true;
	}

	/**
	 * Attempts to creates a forkable thread (see FForkProcessHelper) with the specified stack size and creates the various
	 * events to be able to communicate with it.
	 * 
	 * Currently only works after the process has been forked because GetSingleThreadInterface() isn't implemented
	 *
	 * @param InPool The thread pool interface used to place this thread back into the pool of available threads when its work is done
	 * @param InStackSize The size of the stack to create. 0 means use the current thread's stack size
	 * @param ThreadPriority priority of new thread
	 * @return True if the thread and all of its initialization was successful, false otherwise
	 */
	virtual bool CreateForkable(class FQueuedThreadPoolBase* InPool, uint32 InStackSize = 0, EThreadPriority ThreadPriority = TPri_Normal, const TCHAR* ThreadName = nullptr)
	{
		const FString PoolThreadName = SetupForCreate(InPool, ThreadName);

		// Currently no single threaded support, so this is expected to fail before forking
		Thread = FForkProcessHelper::CreateForkableThread(this, *PoolThreadName, InStackSize, ThreadPriority, FPlatformAffinity::GetPoolThreadMask());
		return !!Thread;
	}
	
	/**
	 * Tells the thread to exit. If the caller needs to know when the thread
	 * has exited, it should use the bShouldWait value and tell it how long
	 * to wait before deciding that it is deadlocked and needs to be destroyed.
	 * NOTE: having a thread forcibly destroyed can cause leaks in TLS, etc.
	 *
	 * @return True if the thread exited graceful, false otherwise
	 */
	bool KillThread()
	{
		bool bDidExitOK = true;
		// Tell the thread it needs to die
		TimeToDie = true;
		// Trigger the thread so that it will come out of the wait state if
		// it isn't actively doing work
		DoWorkEvent->Trigger();
		// If waiting was specified, wait the amount of time. If that fails,
		// brute force kill that thread. Very bad as that might leak.
		Thread->WaitForCompletion();
		// Clean up the event
		FPlatformProcess::ReturnSynchEventToPool(DoWorkEvent);
		DoWorkEvent = nullptr;
		delete Thread;
		return bDidExitOK;
	}

	/**
	 * Tells the thread there is work to be done. Upon completion, the thread
	 * is responsible for adding itself back into the available pool.
	 *
	 * @param InQueuedWork The queued work to perform
	 */
	void DoWork(IQueuedWork* InQueuedWork)
	{
		DECLARE_SCOPE_CYCLE_COUNTER( TEXT( "FQueuedThread::DoWork" ), STAT_FQueuedThread_DoWork, STATGROUP_ThreadPoolAsyncTasks );

		check(QueuedWork == nullptr && "Can't do more than one task at a time");
		// Tell the thread the work to be done
		QueuedWork = InQueuedWork;
		FPlatformMisc::MemoryBarrier();
		// Tell the thread to wake up and do its job
		DoWorkEvent->Trigger();
	}
};


/**
 * Implementation of a queued thread pool.
 */
class FQueuedThreadPoolBase : public FQueuedThreadPool
{
protected:

	/** The work queue to pull from. */
	FThreadPoolPriorityQueue QueuedWork;
	
	/** The thread pool to dole work out to. */
	TArray<FQueuedThread*> QueuedThreads;

	/** All threads in the pool. */
	TArray<FQueuedThread*> AllThreads;

	/** The synchronization object used to protect access to the queued work. */
	FCriticalSection* SynchQueue;

	/** If true, indicates the destruction process has taken place. */
	bool TimeToDie;

public:

	/** Default constructor. */
	FQueuedThreadPoolBase()
		: SynchQueue(nullptr)
		, TimeToDie(0)
	{ }

	/** Virtual destructor (cleans up the synchronization objects). */
	virtual ~FQueuedThreadPoolBase()
	{
		Destroy();
	}

	bool CreateInternal(bool bForkable, uint32 InNumQueuedThreads, uint32 StackSize, EThreadPriority ThreadPriority, const TCHAR* Name)
	{
		UE::Trace::ThreadGroupBegin(Name);

		// Make sure we have synch objects
		bool bWasSuccessful = true;
		check(SynchQueue == nullptr);
		SynchQueue = new FCriticalSection();
		FScopeLock Lock(SynchQueue);
		// Presize the array so there is no extra memory allocated
		check(QueuedThreads.Num() == 0);
		QueuedThreads.Empty(InNumQueuedThreads);
		QueuedWork.Reset();

		// Check for stack size override.
		if( OverrideStackSize > StackSize )
		{
			StackSize = OverrideStackSize;
		}

		// Now create each thread and add it to the array
		for (uint32 Count = 0; Count < InNumQueuedThreads && bWasSuccessful == true; Count++)
		{
			// Create a new queued thread
			FQueuedThread* pThread = new FQueuedThread();
			// Now create the thread and add it if ok
			const FString ThreadName = FString::Printf(TEXT("%s #%d"), Name, Count);
			const bool bCreateSuccess = bForkable ?
				pThread->CreateForkable(this, StackSize, ThreadPriority, *ThreadName) :
				pThread->Create(this, StackSize, ThreadPriority, *ThreadName);
			if (bCreateSuccess)
			{
				QueuedThreads.Add(pThread);
				AllThreads.Add(pThread);
			}
			else
			{
				// Failed to fully create so clean up
				bWasSuccessful = false;
				delete pThread;
			}
		}
		// Destroy any created threads if the full set was not successful
		if (bWasSuccessful == false)
		{
			Destroy();
		}

		UE::Trace::ThreadGroupEnd();
		return bWasSuccessful;
	}

	virtual bool Create(uint32 InNumQueuedThreads, uint32 StackSize, EThreadPriority ThreadPriority, const TCHAR* Name) override
	{
		return CreateInternal(false, InNumQueuedThreads, StackSize, ThreadPriority, Name);
	}

	virtual bool CreateForkable(uint32 InNumQueuedThreads, uint32 StackSize, EThreadPriority ThreadPriority, const TCHAR* Name) override
	{
		return CreateInternal(true, InNumQueuedThreads, StackSize, ThreadPriority, Name);
	}

	virtual void Destroy() override final
	{
		if (SynchQueue)
		{
			{
				FScopeLock Lock(SynchQueue);
				TimeToDie = 1;
				FPlatformMisc::MemoryBarrier();
				// Clean up all queued objects
				while (IQueuedWork * WorkItem = QueuedWork.Dequeue())
				{
					WorkItem->Abandon();
				}
				
				QueuedWork.Reset();
			}
			// wait for all threads to finish up
			while (1)
			{
				{
					FScopeLock Lock(SynchQueue);
					if (AllThreads.Num() == QueuedThreads.Num())
					{
						break;
					}
				}
				FPlatformProcess::Sleep(0.0f);
			}
			// Delete all threads
			{
				FScopeLock Lock(SynchQueue);
				// Now tell each thread to die and delete those
				for (int32 Index = 0; Index < AllThreads.Num(); Index++)
				{
					AllThreads[Index]->KillThread();
					delete AllThreads[Index];
				}
				QueuedThreads.Empty();
				AllThreads.Empty();
			}
			delete SynchQueue;
			SynchQueue = nullptr;
		}
	}

	int32 GetNumQueuedJobs() const
	{
		// this is a estimate of the number of queued jobs. 
		return QueuedWork.Num();
	}

	virtual int32 GetNumThreads() const 
	{
		return AllThreads.Num();
	}

	void AddQueuedWork(IQueuedWork* InQueuedWork, EQueuedWorkPriority InQueuedWorkPriority) override
	{
		check(InQueuedWork != nullptr);

		if (TimeToDie)
		{
			InQueuedWork->Abandon();
			return;
		}

		// Check to see if a thread is available. Make sure no other threads
		// can manipulate the thread pool while we do this.
		//
		// We pick a thread from the back of the array since this will be the
		// most recently used thread and therefore the most likely to have
		// a 'hot' cache for the stack etc (similar to Windows IOCP scheduling
		// strategy). Picking from the back also happens to be cheaper since
		// no memory movement is necessary.

		check(SynchQueue);

		FQueuedThread* Thread = nullptr;

		{
			FScopeLock sl(SynchQueue);
			const int32 AvailableThreadCount = QueuedThreads.Num();
			if (AvailableThreadCount == 0)
			{
				// No thread available, queue the work to be done
				// as soon as one does become available
				QueuedWork.Enqueue(InQueuedWork, InQueuedWorkPriority);
				return;
			}

			const int32 ThreadIndex = AvailableThreadCount - 1;

			Thread = QueuedThreads[ThreadIndex];
			// Remove it from the list so no one else grabs it
			QueuedThreads.RemoveAt(ThreadIndex, EAllowShrinking::No);
		}

		// Tell our chosen thread to do the work
		Thread->DoWork(InQueuedWork);
	}

	virtual bool RetractQueuedWork(IQueuedWork* InQueuedWork) override
	{
		if (TimeToDie)
		{
			return false; // no special consideration for this, refuse the retraction and let shutdown proceed
		}
		check(InQueuedWork != nullptr);
		check(SynchQueue);
		FScopeLock sl(SynchQueue);
		return QueuedWork.Retract(InQueuedWork);
	}

	IQueuedWork* ReturnToPoolOrGetNextJob(FQueuedThread* InQueuedThread)
	{
		check(InQueuedThread != nullptr);
		IQueuedWork* Work = nullptr;
		// Check to see if there is any work to be done
		FScopeLock sl(SynchQueue);
		if (TimeToDie)
		{
			check(!QueuedWork.Num());  // we better not have anything if we are dying
		}
		
		Work = QueuedWork.Dequeue();

		if (!Work)
		{
			// There was no work to be done, so add the thread to the pool
			QueuedThreads.Add(InQueuedThread);
		}
		return Work;
	}
};

uint32 FQueuedThreadPool::OverrideStackSize = 0;

FQueuedThreadPool* FQueuedThreadPool::Allocate()
{
	return new FQueuedThreadPoolBase;
}

FQueuedThreadPool::FQueuedThreadPool() = default;
FQueuedThreadPool::~FQueuedThreadPool() = default;

//////////////////////////////////////////////////////////////////////////

uint32
FQueuedThread::Run()
{
	while (!TimeToDie.Load(EMemoryOrder::Relaxed))
	{
		// This will force sending the stats packet from the previous frame.
		SET_DWORD_STAT(STAT_ThreadPoolDummyCounter, 0);
		// We need to wait for shorter amount of time
		bool bContinueWaiting = true;

		// Unless we're collecting stats there doesn't appear to be any reason to wake
		// up again until there's work to do (or it's time to die)

#if STATS
		if (FThreadStats::IsCollectingData())
		{
			while (bContinueWaiting)
			{
				DECLARE_CYCLE_STAT_WITH_FLAGS(TEXT("FQueuedThread::Run.WaitForWork"),
				STAT_FQueuedThread_Run_WaitForWork, STATGROUP_ThreadPoolAsyncTasks,
					EStatFlags::Verbose);

				SCOPE_CYCLE_COUNTER(STAT_FQueuedThread_Run_WaitForWork);

				// Wait for some work to do

				bContinueWaiting = !DoWorkEvent->Wait(GDoPooledThreadWaitTimeouts ? 10 : MAX_uint32);
			}
		}
#endif

		if (bContinueWaiting)
		{
			DoWorkEvent->Wait();
		}

		IQueuedWork* LocalQueuedWork = QueuedWork;
		QueuedWork = nullptr;
		FPlatformMisc::MemoryBarrier();
		check(LocalQueuedWork || TimeToDie.Load(EMemoryOrder::Relaxed)); // well you woke me up, where is the job or termination request?
		while (LocalQueuedWork)
		{
			// Tell the object to do the work
			LocalQueuedWork->DoThreadedWork();
			// Let the object cleanup before we remove our ref to it
			LocalQueuedWork = OwningThreadPool->ReturnToPoolOrGetNextJob(this);
		}
	}
	return 0;
}

/*-----------------------------------------------------------------------------
	FThreadSingletonInitializer
-----------------------------------------------------------------------------*/

FTlsAutoCleanup* FThreadSingletonInitializer::Get( TFunctionRef<FTlsAutoCleanup*()> CreateInstance, uint32& InOutTlsSlot )
{
	uint32 TlsSlot;
	FTlsAutoCleanup* ThreadSingleton = nullptr;

	// This open block is a) hit frequently and b) constrained in the scope of
	// open code that is touched.
	// When running with the AutoRTFM sanitizer on this becomes a performance
	// bottleneck even though there is no additional safety achieved. Thus we
	// run this open with the AutoRTFM sanitizer disabled.
	UE_AUTORTFM_OPEN_NO_SANITIZE
	{
		TlsSlot = (uint32)FPlatformAtomics::AtomicRead_Relaxed((int32*)&InOutTlsSlot);
		if (TlsSlot == FPlatformTLS::InvalidTlsSlot)
		{
			const uint32 ThisTlsSlot = FPlatformTLS::AllocTlsSlot();
			check(FPlatformTLS::IsValidTlsSlot(ThisTlsSlot));
			const uint32 PrevTlsSlot = FPlatformAtomics::InterlockedCompareExchange( (int32*)&InOutTlsSlot, (int32)ThisTlsSlot, FPlatformTLS::InvalidTlsSlot );
			if (PrevTlsSlot != FPlatformTLS::InvalidTlsSlot)
			{
				FPlatformTLS::FreeTlsSlot( ThisTlsSlot );
				TlsSlot = PrevTlsSlot;
			}
			else
			{
				TlsSlot = ThisTlsSlot;
			}
		}

		ThreadSingleton = (FTlsAutoCleanup*)FPlatformTLS::GetTlsValue( TlsSlot );
		if( !ThreadSingleton )
		{
			ThreadSingleton = CreateInstance();
			ThreadSingleton->Register();
			FPlatformTLS::SetTlsValue( TlsSlot, ThreadSingleton );
		}
	};
	return ThreadSingleton;
}

FTlsAutoCleanup* FThreadSingletonInitializer::TryGet(uint32& TlsSlot)
{
	if (TlsSlot == FPlatformTLS::InvalidTlsSlot)
	{
		return nullptr;
	}

	FTlsAutoCleanup* ThreadSingleton = (FTlsAutoCleanup*)FPlatformTLS::GetTlsValue(TlsSlot);
	return ThreadSingleton;
}

FTlsAutoCleanup* FThreadSingletonInitializer::Inject(FTlsAutoCleanup* Instance, uint32& TlsSlot)
{
	if (TlsSlot == FPlatformTLS::InvalidTlsSlot)
	{
		const uint32 ThisTlsSlot = FPlatformTLS::AllocTlsSlot();
		check(FPlatformTLS::IsValidTlsSlot(ThisTlsSlot));
		const uint32 PrevTlsSlot = FPlatformAtomics::InterlockedCompareExchange( (int32*)&TlsSlot, (int32)ThisTlsSlot, FPlatformTLS::InvalidTlsSlot);
		if (PrevTlsSlot != FPlatformTLS::InvalidTlsSlot)
		{
			FPlatformTLS::FreeTlsSlot( ThisTlsSlot );
		}
	}

	FTlsAutoCleanup* ThreadSingleton = (FTlsAutoCleanup*)FPlatformTLS::GetTlsValue(TlsSlot);
	FPlatformTLS::SetTlsValue(TlsSlot, Instance);
	return ThreadSingleton;
}

void FTlsAutoCleanup::Register()
{
	// Destruction of TLS values need to occur in the reverse order of initialization, as thread_local does.
	struct FTlsContainer : public TArray<TUniquePtr<FTlsAutoCleanup>>
	{
		~FTlsContainer()
		{
			for (int32 Index = Num() - 1; Index >= 0; Index--)
			{
				(*this)[Index].Reset();
			}
		}
	};

	static thread_local FTlsContainer TlsInstances;
	TlsInstances.Add(TUniquePtr<FTlsAutoCleanup>(this));
}

//-------------------------------------------------------------------------------
// FForkableThread
//-------------------------------------------------------------------------------

/**
 * This thread starts as a fake thread and gets ticked like it was in a single-threaded environment.
 * Once it receives the OnPostFork event it creates and holds a real thread that
 * will cause the RunnableObject to be executed in it's own thread.
 * Optionally, it can create the real thread immediately, but it also asserts that the real thread has exited by the fork point.
 */
class FForkableThread : public FFakeThread
{
	typedef FFakeThread Super;

private:

	/** Real thread that gets created right after forking */
	FRunnableThread* RealThread = nullptr;

	/** Cached values to use when the real thread is created post-fork */
	EThreadPriority CachedPriority = TPri_Normal;
	uint32 CachedStackSize = 0;

public:

	FForkableThread(bool InAllowPreForkRealThread)
		: bAllowPreForkRealThread(InAllowPreForkRealThread)
	{}

	virtual ~FForkableThread()
	{
		delete RealThread;
		RealThread = nullptr;
	}

	virtual void Tick() override
	{
		// Tick in single-thread mode when the real thread isn't created yet
		if(RealThread == nullptr)
		{
			Super::Tick();
		}
	}

	virtual void SetThreadPriority(EThreadPriority NewPriority) override
	{
		CachedPriority = NewPriority;
		
		if (RealThread)
		{
			RealThread->SetThreadPriority(NewPriority);
		}
	}

	virtual void Suspend(bool bShouldPause) override
	{
		Super::Suspend(bShouldPause);

		if (RealThread)
		{
			RealThread->Suspend(bShouldPause);
		}
	}

	virtual bool Kill(bool bShouldWait) override
	{
		bool bExitedCorrectly = true;

		if (RealThread)
		{
			bExitedCorrectly = RealThread->Kill(bShouldWait);
		}

		Super::Kill(bShouldWait);

		return bExitedCorrectly;
	}

	virtual void WaitForCompletion() override
	{
		if (RealThread)
		{
			RealThread->WaitForCompletion();
		}

		Super::WaitForCompletion();
	}

	virtual FRunnableThread::ThreadType GetThreadType() const override
	{
		return ThreadType::Forkable;
	}

	virtual bool CreateInternal(FRunnable* InRunnable, const TCHAR* InThreadName, uint32 InStackSize, EThreadPriority InThreadPri, uint64 InThreadAffinityMask, EThreadCreateFlags InCreateFlags) override
	{
		checkf(FForkProcessHelper::SupportsMultithreadingPostFork(), TEXT("ForkableThreads should only be created when -PostForkThreading is enabled"));
		checkf(FForkProcessHelper::IsForkedMultithreadInstance() == false, TEXT("Once forked we create a real runnable thread instead of a ForkableThread"));

		bool bCreated = false;
		if (bAllowPreForkRealThread)
		{
			ThreadName = InThreadName;
			ThreadAffinityMask = InThreadAffinityMask;
			Runnable = InRunnable;
			InRunnable->Init();
			bCreated = CreateRealThread();
		}
		else
		{
			// Call the fake thread creator
			bCreated = Super::CreateInternal(InRunnable, InThreadName, InStackSize, InThreadPri, InThreadAffinityMask, InCreateFlags);
		}

		// Cache the target values until we create the real thread
		CachedStackSize = InStackSize;
		CachedPriority = InThreadPri;

		return bCreated;
	}

protected:

	virtual void OnPreFork() override
	{
		checkf(!bAllowPreForkRealThread, TEXT("A forkable thread that allows real threading before the fork point was left alive. You must delete all forkable threads that were created with bAllowPrefork == true before forking. ThreadName:%s ThreadID:%d"), *GetThreadName(), GetThreadID());
	}

	virtual void OnPostFork() override
	{
		check(FForkProcessHelper::IsForkedMultithreadInstance());
		CreateRealThread();
	}

	bool CreateRealThread()
	{
		check(RealThread == nullptr);
		RealThread = FPlatformProcess::CreateRunnableThread();
		bool bCreated = RealThread->CreateInternal(Runnable, *GetThreadName(), CachedStackSize, CachedPriority, ThreadAffinityMask, EThreadCreateFlags::None);

		if (bCreated)
		{
			RealThread->PostCreate(CachedPriority);

			// Suspend the thread if the fake thread was suspended too
			//TODO: this lets the thread run for a few cycles before hitting the suspend call...
			if (bIsSuspended)
			{
				RealThread->Suspend(bIsSuspended);
			}
		}
		else
		{
			delete RealThread;
			RealThread = nullptr;
		}

		return bCreated;
	}

	bool bAllowPreForkRealThread;
};

FRunnableThread* FForkProcessHelper::CreateForkableThread(class FRunnable* InRunnable, const TCHAR* InThreadName, uint32 InStackSize, EThreadPriority InThreadPri, uint64 InThreadAffinityMask, EThreadCreateFlags InCreateFlags, bool bAllowPrefork)
{
	bool bCreateRealThread = FPlatformProcess::SupportsMultithreading();
	bool bCreateForkableThread(false);

	// Look for conditions allowing real threads in a non-multithread environment
	if (bCreateRealThread == false)
	{
		if( SupportsMultithreadingPostFork() )
		{
			if( IsForkedMultithreadInstance() )
			{
				// Already forked, create a real thread immediately
				bCreateRealThread = true;
			}
			else
			{
				// We have yet to fork the process, create a forkable thread to handle the fork event
				bCreateForkableThread = true;
			}
		
		}
	}

	FRunnableThread* NewThread(nullptr);
	if (bCreateRealThread)
	{
		check(InRunnable);
		NewThread = FPlatformProcess::CreateRunnableThread();
	}
	else if (bCreateForkableThread)
	{
		if (bAllowPrefork)
		{
			NewThread = new FForkableThread(true);
		}
		else if( InRunnable->GetSingleThreadInterface() )
		{
			NewThread = new FForkableThread(false);
		}
	}
	else
	{
		if (InRunnable->GetSingleThreadInterface())
		{
			NewThread = new FFakeThread();
		}
	}

	if (NewThread)
	{
		FRunnableThread::SetupCreatedThread(NewThread, InRunnable, InThreadName, InStackSize, InThreadPri, InThreadAffinityMask, InCreateFlags);
	}

	return NewThread;
}
