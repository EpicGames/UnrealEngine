// Copyright Epic Games, Inc. All Rights Reserved.

#include "HAL/UnrealMemory.h"
#include "Math/UnrealMathUtility.h"
#include "Containers/Array.h"
#include "Logging/LogMacros.h"
#include "HAL/ThreadSafeCounter.h"
#include "Containers/LockFreeList.h"
#include "Stats/Stats.h"
#include "HAL/IConsoleManager.h"
#include "Misc/CoreDelegates.h"
#include "ProfilingDebugging/CsvProfiler.h"

/*-----------------------------------------------------------------------------
	Memory functions.
-----------------------------------------------------------------------------*/

#include "ProfilingDebugging/MemoryTrace.h"
#include "HAL/MallocThreadSafeProxy.h"
#include "HAL/MallocVerify.h"
#include "HAL/MallocLeakDetectionProxy.h"
#include "HAL/PlatformMallocCrash.h"
#include "HAL/MallocPoisonProxy.h"
#include "HAL/MallocDoubleFreeFinder.h"
#include "HAL/MallocFrameProfiler.h"
#include "HAL/MallocStomp2.h"
#include "HAL/Memory.h"

CSV_DEFINE_CATEGORY_MODULE(CORE_API, FMemory, true);

#if MALLOC_GT_HOOKS

// This code is used to find memory allocations, you set up the lambda in the section of the code you are interested in and add a breakpoint to your lambda to see who is allocating memory

//An example:
//static TFunction<void(int32)> AnimHook(
//	[](int32 Index)
//{
//	TickAnimationMallocStats[Index]++;
//	if (++AllCount % 337 == 0)
//	{
//		BreakMe();
//	}
//}
//);
//extern CORE_API TFunction<void(int32)>* GGameThreadMallocHook;
//TGuardValue<TFunction<void(int32)>*> RestoreHook(GGameThreadMallocHook, &AnimHook);


CORE_API TFunction<void(int32)>* GGameThreadMallocHook = nullptr;

void DoGamethreadHook(int32 Index)
{
	if (GIsRunning && GGameThreadMallocHook // otherwise our hook might not be initialized yet
		&& IsInGameThread())
	{
		(*GGameThreadMallocHook)(Index);
	}
}
#endif

class FMallocPurgatoryProxy : public FMalloc
{
	/** Malloc we're based on, aka using under the hood							*/
	FMalloc*			UsedMalloc;
	enum
	{
		PURGATORY_STOMP_CHECKS_FRAMES = 4, // we want to allow several frames since it is theoretically possible for something to be blocked mid-allocation for many frames.
		PURGATORY_STOMP_MAX_PURGATORY_MEM_KB = 100000, // for loading and whatnot, we really can't keep this stuff forever
		PURGATORY_STOMP_CHECKS_CANARYBYTE = 0xdc,
	};
	uint32 LastCheckFrame; // unitialized; won't matter
	FThreadSafeCounter OutstandingSizeInKB;
	FThreadSafeCounter NextOversizeClear;
	TLockFreePointerListUnordered<void, PLATFORM_CACHE_LINE_SIZE> Purgatory[PURGATORY_STOMP_CHECKS_FRAMES];

public:
	/**
	* Constructor for thread safe proxy malloc that takes a malloc to be used and a
	* synchronization object used via FScopeLock as a parameter.
	*
	* @param	InMalloc					FMalloc that is going to be used for actual allocations
	*/
	FMallocPurgatoryProxy(FMalloc* InMalloc)
		: UsedMalloc(InMalloc)
	{}

	virtual void InitializeStatsMetadata() override
	{
		UsedMalloc->InitializeStatsMetadata();
	}

	/**
	* Malloc
	*/
	 void* Malloc(SIZE_T Size, uint32 Alignment) 
	{
		return UsedMalloc->Malloc(Size, Alignment);
	}

	/**
	* Realloc
	*/
	virtual void* Realloc(void* Ptr, SIZE_T NewSize, uint32 Alignment) override
	{
		return UsedMalloc->Realloc(Ptr, NewSize, Alignment);
	}

	/**
	* Free
	*/
	virtual void Free(void* Ptr) override
	{
		if (!Ptr)// || !GIsGameThreadIdInitialized)
		{
			UsedMalloc->Free(Ptr);
			return;
		}
		{
			SIZE_T Size = 0;
			verify(GetAllocationSize(Ptr, Size) && Size);
			FMemory::Memset(Ptr, uint8(PURGATORY_STOMP_CHECKS_CANARYBYTE), Size);
			Purgatory[GFrameNumber % PURGATORY_STOMP_CHECKS_FRAMES].Push(Ptr);
			OutstandingSizeInKB.Add((int32)((Size + 1023) / 1024));
		}
		FPlatformMisc::MemoryBarrier();
		uint32 LocalLastCheckFrame = LastCheckFrame;
		uint32 LocalGFrameNumber = GFrameNumber;

		bool bFlushAnyway = OutstandingSizeInKB.GetValue() > PURGATORY_STOMP_MAX_PURGATORY_MEM_KB;

		if (bFlushAnyway || LocalLastCheckFrame != LocalGFrameNumber)
		{
			if (bFlushAnyway || FPlatformAtomics::InterlockedCompareExchange((volatile int32*)&LastCheckFrame, LocalGFrameNumber, LocalLastCheckFrame) == LocalLastCheckFrame)
			{
				int32 FrameToPop = ((bFlushAnyway ? NextOversizeClear.Increment() : LocalGFrameNumber) + PURGATORY_STOMP_CHECKS_FRAMES - 1) % PURGATORY_STOMP_CHECKS_FRAMES;
				//FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Pop %d at %d %d\r\n"), FrameToPop, LocalGFrameNumber, LocalGFrameNumber % PURGATORY_STOMP_CHECKS_FRAMES);
				while (true)
				{
					uint8* Pop = (uint8*)Purgatory[FrameToPop].Pop();
					if (!Pop)
					{
						break;
					}
					SIZE_T Size = 0;
					verify(GetAllocationSize(Pop, Size) && Size);
					for (SIZE_T At = 0; At < Size; At++)
					{
						if (Pop[At] != PURGATORY_STOMP_CHECKS_CANARYBYTE)
						{
							FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Freed memory at %p + %d == %x (should be %x)\r\n"), Pop, At, (int32)Pop[At], (int32)PURGATORY_STOMP_CHECKS_CANARYBYTE);
							UE_LOG(LogMemory, Fatal, TEXT("Freed memory at %p + %" SIZE_T_FMT " == %x (should be %x)"), Pop, At, (int32)Pop[At], (int32)PURGATORY_STOMP_CHECKS_CANARYBYTE);
						}
					}
					UsedMalloc->Free(Pop);
					OutstandingSizeInKB.Subtract((int32)((Size + 1023) / 1024));
				}
			}
		}
	}

	/** Writes allocator stats from the last update into the specified destination. */
	virtual void GetAllocatorStats(FGenericMemoryStats& out_Stats) override
	{
		UsedMalloc->GetAllocatorStats(out_Stats);
	}

	/** Dumps allocator stats to an output device. */
	virtual void DumpAllocatorStats(class FOutputDevice& Ar) override
	{
		UsedMalloc->DumpAllocatorStats(Ar);
	}

	/**
	* Validates the allocator's heap
	*/
	virtual bool ValidateHeap() override
	{
		return(UsedMalloc->ValidateHeap());
	}

#if UE_ALLOW_EXEC_COMMANDS
	virtual bool Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override
	{
		return UsedMalloc->Exec(InWorld, Cmd, Ar);
	}
#endif // UE_ALLOW_EXEC_COMMANDS

	/**
	* If possible determine the size of the memory allocated at the given address
	*
	* @param Original - Pointer to memory we are checking the size of
	* @param SizeOut - If possible, this value is set to the size of the passed in pointer
	* @return true if succeeded
	*/
	virtual bool GetAllocationSize(void *Original, SIZE_T &SizeOut) override
	{
		return UsedMalloc->GetAllocationSize(Original, SizeOut);
	}

	virtual SIZE_T QuantizeSize(SIZE_T Count, uint32 Alignment) override
	{
		return UsedMalloc->QuantizeSize(Count, Alignment);
	}
	virtual void Trim(bool bTrimThreadCaches) override
	{
		return UsedMalloc->Trim(bTrimThreadCaches);
	}
	virtual void SetupTLSCachesOnCurrentThread() override
	{
		return UsedMalloc->SetupTLSCachesOnCurrentThread();
	}
	virtual void MarkTLSCachesAsUsedOnCurrentThread() override
	{
		UsedMalloc->MarkTLSCachesAsUsedOnCurrentThread();
	}
	virtual void MarkTLSCachesAsUnusedOnCurrentThread() override
	{
		UsedMalloc->MarkTLSCachesAsUnusedOnCurrentThread();
	}
	virtual void ClearAndDisableTLSCachesOnCurrentThread() override
	{
		return UsedMalloc->ClearAndDisableTLSCachesOnCurrentThread();
	}
	virtual const TCHAR* GetDescriptiveName() override
	{
		return UsedMalloc->GetDescriptiveName();
	}

	virtual void OnMallocInitialized() override
	{
		UsedMalloc->OnMallocInitialized();
	}

	virtual void OnPreFork() override
	{
		UsedMalloc->OnPreFork();
	}

	virtual void OnPostFork() override
	{
		UsedMalloc->OnPostFork();
	}
};

void FMemory::EnablePurgatoryTests()
{
	if (PLATFORM_USES_FIXED_GMalloc_CLASS)
	{
		UE_LOGF(LogMemory, Error, "Purgatory proxy cannot be turned on because we are using PLATFORM_USES_FIXED_GMalloc_CLASS");
		return;
	}
	static bool bOnce = false;
	if (bOnce)
	{
		UE_LOGF(LogMemory, Error, "Purgatory proxy was already turned on.");
		return;
	}
	bOnce = true;
	while (true)
	{
		FMalloc* LocalGMalloc = UE::Private::GMalloc;
		FMalloc* Proxy = new FMallocPurgatoryProxy(LocalGMalloc);
		if (FPlatformAtomics::InterlockedCompareExchangePointer((void**)&UE::Private::GMalloc, Proxy, LocalGMalloc) == LocalGMalloc)
		{
			UE_LOGF(LogConsoleResponse, Display, "Purgatory proxy is now on.");
			return;
		}
		delete Proxy;
	}
}

void FMemory::EnablePoisonTests()
{
	if ( ! FPlatformProcess::SupportsMultithreading() )
	{
		UE_LOGF(LogConsoleResponse, Display, "SKIPPING Poison proxy - platform does not support multithreads");
		return;
	}
	if (PLATFORM_USES_FIXED_GMalloc_CLASS)
	{
		UE_LOGF(LogMemory, Error, "Poison proxy cannot be turned on because we are using PLATFORM_USES_FIXED_GMalloc_CLASS");
		return;
	}
	static bool bOnce = false;
	if (bOnce)
	{
		UE_LOGF(LogMemory, Error, "Poison proxy was already turned on.");
		return;
	}
	bOnce = true;
	while (true)
	{
		FMalloc* LocalGMalloc = UE::Private::GMalloc;
		FMalloc* Proxy = new FMallocPoisonProxy(LocalGMalloc);
		if (FPlatformAtomics::InterlockedCompareExchangePointer((void**)&UE::Private::GMalloc, Proxy, LocalGMalloc) == LocalGMalloc)
		{
			UE_LOGF(LogConsoleResponse, Display, "Poison proxy is now on.");
			return;
		}
		delete Proxy;
	}
}

#if !UE_BUILD_SHIPPING
#include "Async/TaskGraphInterfaces.h"
static void FMallocBinnedOverrunTest()
{
	const uint32 ArraySize = 64;
	uint8* Pointer = new uint8[ArraySize];
	delete[] Pointer;
	FFunctionGraphTask::CreateAndDispatchWhenReady(
		[Pointer, ArraySize]()
	{
		//FPlatformProcess::Sleep(.01f);
		Pointer[ArraySize / 2] = 0xcc; //-V774
	},
		TStatId()
		);
}

FAutoConsoleCommand FMallocBinnedTestCommand
(
TEXT("Memory.StaleTest"),
TEXT("Test for Memory.UsePurgatory. *** Will crash the game!"),
FConsoleCommandDelegate::CreateStatic(&FMallocBinnedOverrunTest)
);

FAutoConsoleCommand FMallocUsePurgatoryCommand
(
TEXT("Memory.UsePurgatory"),
TEXT("Uses the purgatory malloc proxy to check if things are writing to stale pointers."),
FConsoleCommandDelegate::CreateStatic(&FMemory::EnablePurgatoryTests)
);

FAutoConsoleCommand FMallocUsePoisonCommand
(
TEXT("Memory.UsePoison"),
TEXT("Uses the poison malloc proxy to check if things are relying on uninitialized or free'd memory."),
FConsoleCommandDelegate::CreateStatic(&FMemory::EnablePoisonTests)
);
#endif



/** Helper function called on first allocation to create and initialize GMalloc */
static int FMemory_GCreateMalloc_ThreadUnsafe()
{
	UE::Private::GMalloc = FPlatformMemory::BaseAllocator();

#if !PLATFORM_MAC
	FPlatformMemoryStats Stats = FPlatformMemory::GetStats();
	uint64 ProgramSize = Stats.UsedPhysical;
#endif

	// Setup malloc crash as soon as possible.
	FPlatformMallocCrash::Get(UE::Private::GMalloc);

#if PLATFORM_USES_FIXED_GMalloc_CLASS
#if MALLOC_VERIFY || MALLOC_LEAKDETECTION || UE_USE_MALLOC_FILL_BYTES
#error "Turn off PLATFORM_USES_FIXED_GMalloc_CLASS in order to use special allocator proxies"
#endif
	if (!UE::Private::GMalloc->IsInternallyThreadSafe())
	{
		UE_LOGF(LogMemory, Fatal, "PLATFORM_USES_FIXED_GMalloc_CLASS only makes sense for allocators that are internally threadsafe.");
	}
#else

#if UE_MEMORY_TRACE_ENABLED
	FMalloc* TraceMalloc = MemoryTrace_Create(UE::Private::GMalloc);
	if (TraceMalloc != UE::Private::GMalloc)
	{
		UE::Private::GMalloc = TraceMalloc;
		MemoryTrace_Initialize();
	}
#endif // UE_MEMORY_TRACE_ENABLED

#if WITH_MALLOC_STOMP2
	UE::Private::GMalloc = FMallocStomp2::OverrideIfEnabled(UE::Private::GMalloc);
#endif

	// if the allocator is already thread safe, there is no need for the thread safe proxy
	if (!UE::Private::GMalloc->IsInternallyThreadSafe())
	{
		UE::Private::GMalloc = new FMallocThreadSafeProxy(UE::Private::GMalloc);
	}

#if MALLOC_VERIFY
	// Add the verifier
	UE::Private::GMalloc = new FMallocVerifyProxy(UE::Private::GMalloc);
#endif

#if MALLOC_LEAKDETECTION
	UE::Private::GMalloc = new FMallocLeakDetectionProxy(UE::Private::GMalloc);
#endif

	// poison memory allocations in Debug and Development non-editor/non-editoronly data builds
#if UE_USE_MALLOC_FILL_BYTES
	UE::Private::GMalloc = new FMallocPoisonProxy(UE::Private::GMalloc);
#endif

#endif

// On Mac it's too early to log here in some cases. For example GMalloc may be created during initialization of a third party dylib on load, before CoreFoundation is initialized
#if !PLATFORM_DESKTOP
	// by this point, we assume we can log
	double SizeInMb = (double)ProgramSize / (1024.0 * 1024.0);
	FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Used memory before allocating anything was %.2fMB\n"), SizeInMb);
	UE_LOGF(LogMemory, Display, "Used memory before allocating anything was %.2fMB", SizeInMb);
#endif

	UE::Private::GMalloc = FMallocDoubleFreeFinder::OverrideIfEnabled(UE::Private::GMalloc);
	UE::Private::GMalloc = FMallocFrameProfiler::OverrideIfEnabled(UE::Private::GMalloc);

	UE::Private::GMalloc->OnMallocInitialized();

	return 0;
}

void FMemory::ExplicitInit(FMalloc& Allocator)
{
#if defined(REQUIRE_EXPLICIT_GMALLOC_INIT) && REQUIRE_EXPLICIT_GMALLOC_INIT
	check(!UE::Private::GMalloc);
	UE::Private::GMalloc = &Allocator;
#else
	checkf(false, TEXT("ExplicitInit() forbidden when global allocator is created lazily"));
#endif
}

void FMemory::GCreateMalloc()
{
#if defined(REQUIRE_EXPLICIT_GMALLOC_INIT) && REQUIRE_EXPLICIT_GMALLOC_INIT
	checkf(false, TEXT("Allocating before ExplicitInit()"));
#else
	// On some platforms (e.g. Mac) GMalloc can be created on multiple threads at once.
	// This admittedly clumsy construct ensures both thread-safe creation and prevents multiple calls into it.
	// The call will not be optimized away in Shipping because the function has side effects (touches global vars).
	static int ThreadSafeCreationResult = FMemory_GCreateMalloc_ThreadUnsafe();
#endif
}

#if TIME_MALLOC

uint64 FScopedMallocTimer::GTotalCycles[4] = { 0 };
uint64 FScopedMallocTimer::GTotalCount[4] = { 0 };
uint64 FScopedMallocTimer::GTotalMisses[4] = { 0 };

void FScopedMallocTimer::Spew()
{
	static uint64 GLastTotalCycles[4] = { 0 };
	static uint64 GLastTotalCount[4] = { 0 };
	static uint64 GLastTotalMisses[4] = { 0 };
	static uint64 GLastFrame = 0;

	uint64 Frames = GFrameCounter - GLastFrame;
	if (Frames)
	{
		GLastFrame = GFrameCounter;
		// not atomic; we assume the error is minor
		uint64 TotalCycles[4] = { 0 };
		uint64 TotalCount[4] = { 0 };
		uint64 TotalMisses[4] = { 0 };
		for (int32 Comp = 0; Comp < 4; Comp++)
		{
			TotalCycles[Comp] = GTotalCycles[Comp] - GLastTotalCycles[Comp];
			TotalCount[Comp] = GTotalCount[Comp] - GLastTotalCount[Comp];
			TotalMisses[Comp] = GTotalMisses[Comp] - GLastTotalMisses[Comp];
			GLastTotalCycles[Comp] = GTotalCycles[Comp];
			GLastTotalCount[Comp] = GTotalCount[Comp];
			GLastTotalMisses[Comp] = GTotalMisses[Comp];
		}
		auto PrintIt = [&TotalCycles, &TotalCount, &TotalMisses, &Frames](const TCHAR * Op, int32 InIndex)
		{
			if (TotalCount[InIndex])
			{
				UE_LOGF(LogMemory, Display, "FMemory %8ls  %5d count/frame   %6.2fms / frame (all threads)  %6.2fns / op    inline miss rate %5.2f%%",
					Op,
					int32(TotalCount[InIndex] / Frames),
					1000.0f * float(FPlatformTime::GetSecondsPerCycle64()) * float(TotalCycles[InIndex]) / float(Frames),
					1000000000.0f * float(FPlatformTime::GetSecondsPerCycle64()) * float(TotalCycles[InIndex]) / float(TotalCount[InIndex]),
					100.0f * float(TotalMisses[InIndex]) / float(TotalCount[InIndex])
					);
			}
		};
		PrintIt(TEXT("Malloc"), 0);
		PrintIt(TEXT("Realloc"), 1);
		PrintIt(TEXT("Free"), 2);
		PrintIt(TEXT("NullFree"), 3);
	}
}

#endif

void* FMemory::MallocExternal(SIZE_T Count, uint32 Alignment)
{
	if (!UE::Private::GMalloc)
	{
		GCreateMalloc();
		CA_ASSUME(UE::Private::GMalloc != nullptr);	// Don't want to assert, but suppress static analysis warnings about potentially NULL GMalloc
	}
	return UE::Private::GMalloc->Malloc(Count, Alignment);
}

void* FMemory::ReallocExternal(void* Original, SIZE_T Count, uint32 Alignment)
{
	if (!UE::Private::GMalloc)
	{
		GCreateMalloc();
		CA_ASSUME(UE::Private::GMalloc != nullptr);	// Don't want to assert, but suppress static analysis warnings about potentially NULL GMalloc
	}
	return UE::Private::GMalloc->Realloc(Original, Count, Alignment);
}

void FMemory::FreeExternal(void* Original)
{
	if (!UE::Private::GMalloc)
	{
		GCreateMalloc();
		CA_ASSUME(UE::Private::GMalloc != nullptr);	// Don't want to assert, but suppress static analysis warnings about potentially NULL GMalloc
	}
	if (Original)
	{
		UE::Private::GMalloc->Free(Original);
	}
}

SIZE_T FMemory::GetAllocSizeExternal(void* Original)
{ 
	if (!UE::Private::GMalloc)
	{
		GCreateMalloc();
		CA_ASSUME(UE::Private::GMalloc != nullptr);	// Don't want to assert, but suppress static analysis warnings about potentially NULL GMalloc
	}
	SIZE_T Size = 0;
	return UE::Private::GMalloc->GetAllocationSize(Original, Size) ? Size : 0;
}

void* FMemory::MallocZeroedExternal(SIZE_T Count, uint32 Alignment)
{
	if (!UE::Private::GMalloc)
	{
		GCreateMalloc();
		CA_ASSUME(UE::Private::GMalloc != nullptr);	// Don't want to assert, but suppress static analysis warnings about potentially NULL GMalloc
	}
	return UE::Private::GMalloc->MallocZeroed(Count, Alignment);
}

SIZE_T FMemory::QuantizeSizeExternal(SIZE_T Count, uint32 Alignment)
{ 
	if (!UE::Private::GMalloc)
	{
		GCreateMalloc();	
		CA_ASSUME(UE::Private::GMalloc != nullptr);	// Don't want to assert, but suppress static analysis warnings about potentially NULL GMalloc
	}
	return UE::Private::GMalloc->QuantizeSize(Count, Alignment);
}	


void FMemory::Trim(bool bTrimThreadCaches)
{
	if (!UE::Private::GMalloc)
	{
		GCreateMalloc();
		CA_ASSUME(UE::Private::GMalloc != nullptr);	// Don't want to assert, but suppress static analysis warnings about potentially NULL GMalloc
	}
	TRACE_CPUPROFILER_EVENT_SCOPE(FMemory::Trim);
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FMemory_Trim);
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FMemory_Trim_Broadcast);
		FCoreDelegates::GetMemoryTrimDelegate().Broadcast();
	}
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FMemory_Trim_GMalloc);
	UE::Private::GMalloc->Trim(bTrimThreadCaches);
}

void FMemory::SetupTLSCachesOnCurrentThread()
{
	if (!UE::Private::GMalloc)
	{
		GCreateMalloc();	
		CA_ASSUME(UE::Private::GMalloc != nullptr);	// Don't want to assert, but suppress static analysis warnings about potentially NULL GMalloc
	}
	UE::Private::GMalloc->SetupTLSCachesOnCurrentThread();
}
	
void FMemory::ClearAndDisableTLSCachesOnCurrentThread()
{
	if (UE::Private::GMalloc)
	{
		UE::Private::GMalloc->ClearAndDisableTLSCachesOnCurrentThread();
	}
}

void FMemory::MarkTLSCachesAsUsedOnCurrentThread()
{
	if (UE::Private::GMalloc)
	{
		UE::Private::GMalloc->MarkTLSCachesAsUsedOnCurrentThread();
	}
}

void FMemory::MarkTLSCachesAsUnusedOnCurrentThread()
{
	if (UE::Private::GMalloc)
	{
		UE::Private::GMalloc->MarkTLSCachesAsUnusedOnCurrentThread();
	}
}

void FMemory::TestMemory()
{
#if !UE_BUILD_SHIPPING
	if( !UE::Private::GMalloc)
	{
		GCreateMalloc();	
		CA_ASSUME(UE::Private::GMalloc != nullptr);	// Don't want to assert, but suppress static analysis warnings about potentially NULL GMalloc
	}

	// track the pointers to free next call to the function
	static TArray<void*> LeakedPointers;
	TArray<void*> SavedLeakedPointers = LeakedPointers;


	// note that at the worst case, there will be NumFreedAllocations + 2 * NumLeakedAllocations allocations alive
	static const int NumFreedAllocations = 1000;
	static const int NumLeakedAllocations = 100;
	static const int MaxAllocationSize = 128 * 1024;

	TArray<void*> FreedPointers;
	// allocate pointers that will be freed later
	for (int32 Index = 0; Index < NumFreedAllocations; Index++)
	{
		FreedPointers.Add(FMemory::Malloc(FMath::RandHelper(MaxAllocationSize)));
	}


	// allocate pointers that will be leaked until the next call
	LeakedPointers.Empty();
	for (int32 Index = 0; Index < NumLeakedAllocations; Index++)
	{
		LeakedPointers.Add(FMemory::Malloc(FMath::RandHelper(MaxAllocationSize)));
	}

	// free the leaked pointers from _last_ call to this function
	for (int32 Index = 0; Index < SavedLeakedPointers.Num(); Index++)
	{
		FMemory::Free(SavedLeakedPointers[Index]);
	}

	// free the non-leaked pointers from this call to this function
	for (int32 Index = 0; Index < FreedPointers.Num(); Index++)
	{
		FMemory::Free(FreedPointers[Index]);
	}
#endif
}

void* FUseSystemMallocForNew::operator new(size_t Size)
{
	return FMemory::SystemMalloc(Size);
}

void FUseSystemMallocForNew::operator delete(void* Ptr)
{
	FMemory::SystemFree(Ptr);
}

void* FUseSystemMallocForNew::operator new[](size_t Size)
{
	return FMemory::SystemMalloc(Size);
}

void FUseSystemMallocForNew::operator delete[](void* Ptr)
{
	FMemory::SystemFree(Ptr);
}

void* FMemory::Malloc(SIZE_T Count, uint32 Alignment)
{
	return FMemory_MallocInline(Count, Alignment);
}

void* FMemory::Realloc(void* Original, SIZE_T Count, uint32 Alignment)
{
	return FMemory_ReallocInline(Original, Count, Alignment);
}

void FMemory::Free(void* Original)
{
	return FMemory_FreeInline(Original);
}

SIZE_T FMemory::GetAllocSize(void* Original)
{
	return FMemory_GetAllocSizeInline(Original);
}

void* FMemory::MallocZeroed(SIZE_T Count, uint32 Alignment)
{
	return FMemory_MallocZeroedInline(Count, Alignment);
}

SIZE_T FMemory::QuantizeSize(SIZE_T Count, uint32 Alignment)
{
	return FMemory_QuantizeSizeInline(Count, Alignment);
}