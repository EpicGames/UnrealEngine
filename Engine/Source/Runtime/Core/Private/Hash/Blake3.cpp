// Copyright Epic Games, Inc. All Rights Reserved.

#include "Hash/Blake3.h"

#include "AutoRTFM.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/UnrealString.h"
#include "Memory/CompositeBuffer.h"
#include "Memory/SharedBuffer.h"
#include "Serialization/Archive.h"
#include "blake3.h"

#if USING_MEMORY_SANITISER
#include <sanitizer/msan_interface.h>
#endif

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

static_assert(sizeof(FBlake3) == sizeof(blake3_hasher), "Adjust the allocation in FBlake3 to match blake3_hasher");

void FBlake3::Reset()
{
	blake3_hasher& Hasher = reinterpret_cast<blake3_hasher&>(HasherBytes);
	// blake3 is a precompiled external library without AutoRTFM support.
	// Call the blake3 function in the open (non-transactionally), and manually
	// record the update to the Hasher structure.
	bool bIsClosed = AutoRTFM::IsClosed();
	UE_AUTORTFM_OPEN
	{
		if (bIsClosed)
		{
			AutoRTFM::RecordOpenWrite(&Hasher, sizeof(Hasher));
		}
		blake3_hasher_init(&Hasher);
	};
}

void FBlake3::Update(FMemoryView View)
{
	Update(View.GetData(), View.GetSize());
}

void FBlake3::Update(const void* Data, uint64 Size)
{
	blake3_hasher& Hasher = reinterpret_cast<blake3_hasher&>(HasherBytes);
	bool bIsClosed = AutoRTFM::IsClosed();
	// blake3 is a precompiled external library without AutoRTFM support.
	// Call the blake3 function in the open (non-transactionally), and manually
	// record the update to the Hasher structure.
	UE_AUTORTFM_OPEN
	{
		if (bIsClosed)
		{
			AutoRTFM::RecordOpenWrite(&Hasher, sizeof(Hasher));
		}
		blake3_hasher_update(&Hasher, Data, Size);
	};
}

void FBlake3::Update(const FCompositeBuffer& Buffer)
{
	for (const FSharedBuffer& Segment : Buffer.GetSegments())
	{
		Update(Segment.GetData(), Segment.GetSize());
	}
}

FBlake3Hash FBlake3::Finalize() const
{
	FBlake3Hash Hash;
	FBlake3Hash::ByteArray& Output = Hash.GetBytes();
	static_assert(sizeof(decltype(Output)) == BLAKE3_OUT_LEN, "Mismatch in BLAKE3 hash size.");
	const blake3_hasher& Hasher = reinterpret_cast<const blake3_hasher&>(HasherBytes);

	UE_AUTORTFM_OPEN
	{
		blake3_hasher_finalize(&Hasher, Output, BLAKE3_OUT_LEN);
	};

#if USING_MEMORY_SANITISER
	// We have to manually unpoison the return from blake3 because it is externally precompiled
	// code (EG. without memory sanitization enabled). We know that it writes to all bytes of
	// our output so we just have to explain to msan here that that is the case.
	__msan_unpoison(Output, sizeof(FBlake3Hash::ByteArray));
#endif

	return Hash;
}

FBlake3Hash FBlake3::HashBuffer(FMemoryView View)
{
	FBlake3 Hash;
	Hash.Update(View.GetData(), View.GetSize());
	return Hash.Finalize();
}

FBlake3Hash FBlake3::HashBuffer(const void* Data, uint64 Size)
{
	FBlake3 Hash;
	Hash.Update(Data, Size);
	return Hash.Finalize();
}

FBlake3Hash FBlake3::HashBuffer(const FCompositeBuffer& Buffer)
{
	FBlake3 Hash;
	Hash.Update(Buffer);
	return Hash.Finalize();
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FArchive& operator<<(FArchive& Ar, FBlake3Hash& Value)
{
	Ar.Serialize(Value.GetBytes(), sizeof(decltype(Value.GetBytes())));
	return Ar;
}

FString LexToString(const FBlake3Hash& Hash)
{
	FString Output;
	TArray<TCHAR, FString::AllocatorType>& CharArray = Output.GetCharArray();
	CharArray.AddUninitialized(sizeof(FBlake3Hash::ByteArray) * 2 + 1);
	UE::String::BytesToHexLower(Hash.GetBytes(), CharArray.GetData());
	CharArray.Last() = TCHAR('\0');
	return Output;
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
