// Copyright (c) 2011 Google, Inc.
//
// Permission is hereby granted, free of charge, to any person obtaining a copy
// of this software and associated documentation files (the "Software"), to deal
// in the Software without restriction, including without limitation the rights
// to use, copy, modify, merge, publish, distribute, sublicense, and/or sell
// copies of the Software, and to permit persons to whom the Software is
// furnished to do so, subject to the following conditions:
//
// The above copyright notice and this permission notice shall be included in
// all copies or substantial portions of the Software.
//
// THE SOFTWARE IS PROVIDED "AS IS", WITHOUT WARRANTY OF ANY KIND, EXPRESS OR
// IMPLIED, INCLUDING BUT NOT LIMITED TO THE WARRANTIES OF MERCHANTABILITY,
// FITNESS FOR A PARTICULAR PURPOSE AND NONINFRINGEMENT. IN NO EVENT SHALL THE
// AUTHORS OR COPYRIGHT HOLDERS BE LIABLE FOR ANY CLAIM, DAMAGES OR OTHER
// LIABILITY, WHETHER IN AN ACTION OF CONTRACT, TORT OR OTHERWISE, ARISING FROM,
// OUT OF OR IN CONNECTION WITH THE SOFTWARE OR THE USE OR OTHER DEALINGS IN
// THE SOFTWARE.
//
// CityHash, by Geoff Pike and Jyrki Alakuijala
//
// This file provides CityHash64() and related functions.
//
// It's probably possible to create even faster hash functions by
// writing a program that systematically explores some of the space of
// possible hash functions, by using SIMD instructions, or by
// compromising on hash quality.

//#include "config.h"
#include "Hash/CityHash.h"
#include "HAL/UnrealMemory.h"


static uint64 UNALIGNED_LOAD64(const char *p) {
	uint64 result;
	FPlatformMemory::Memcpy(&result, p, sizeof(result));
	return result;
}

static uint32 UNALIGNED_LOAD32(const char *p) {
	uint32 result;
	FPlatformMemory::Memcpy(&result, p, sizeof(result));
	return result;
}

#ifdef _MSC_VER

#include <stdlib.h>
#define bswap_32(x) _byteswap_ulong(x)
#define bswap_64(x) _byteswap_uint64(x)

#elif defined(__APPLE__)

// Mac OS X / Darwin features
#include <libkern/OSByteOrder.h>
#define bswap_32(x) OSSwapInt32(x)
#define bswap_64(x) OSSwapInt64(x)

#elif defined(__sun) || defined(sun)

#include <sys/byteorder.h>
#define bswap_32(x) BSWAP_32(x)
#define bswap_64(x) BSWAP_64(x)

#elif defined(__FreeBSD__)

#include <machine/endian.h>
#define bswap_32(x) __bswap32_var(x)
#define bswap_64(x) __bswap64_var(x)

#elif defined(__OpenBSD__)

#include <sys/types.h>
#define bswap_32(x) swap32(x)
#define bswap_64(x) swap64(x)

#elif defined(__NetBSD__)

#include <sys/types.h>
#include <machine/bswap.h>
#if defined(__BSWAP_RENAME) && !defined(__bswap_32)
#define bswap_32(x) bswap32(x)
#define bswap_64(x) bswap64(x)
#endif

#else

#include <Misc/ByteSwap.h>
#define bswap_32(x) BYTESWAP_ORDER32(x)
#define bswap_64(x) BYTESWAP_ORDER64(x)

#endif

#ifdef WORDS_BIGENDIAN
#define uint32_in_expected_order(x) (bswap_32(x))
#define uint64_in_expected_order(x) (bswap_64(x))
#else
#define uint32_in_expected_order(x) (x)
#define uint64_in_expected_order(x) (x)
#endif

#if !defined(LIKELY)
#if HAVE_BUILTIN_EXPECT
#define LIKELY(x) (__builtin_expect(!!(x), 1))
#else
#define LIKELY(x) (x)
#endif
#endif

static uint64 Fetch64(const char *p) {
	return uint64_in_expected_order(UNALIGNED_LOAD64(p));
}

static uint32 Fetch32(const char *p) {
	return uint32_in_expected_order(UNALIGNED_LOAD32(p));
}

namespace CityHash_Internal 
{
	// Some primes between 2^63 and 2^64 for various uses.
	static const uint64 k0 = 0xc3a5c85c97cb3127ULL;
	static const uint64 k1 = 0xb492b66fbe98f273ULL;
	static const uint64 k2 = 0x9ae16a3b2f90404fULL;

	// Magic numbers for 32-bit hashing.  Copied from Murmur3.
	static const uint32 c1 = 0xcc9e2d51;
	static const uint32 c2 = 0x1b873593;
}

// A 32-bit to 32-bit integer hash copied from Murmur3.
static uint32 fmix(uint32 h)
{
	h ^= h >> 16;
	h *= 0x85ebca6b;
	h ^= h >> 13;
	h *= 0xc2b2ae35;
	h ^= h >> 16;
	return h;
}

static uint32 Rotate32(uint32 val, int shift) {
	// Avoid shifting by 32: doing so yields an undefined result.
	return shift == 0 ? val : ((val >> shift) | (val << (32 - shift)));
}

template<typename T>
void SwapValues(T& a, T& b)
{
	T c = a;
	a = b;
	b = c;
}

#undef PERMUTE3
#define PERMUTE3(a, b, c) do { SwapValues(a, b); SwapValues(a, c); } while (0)

static uint32 Mur(uint32 a, uint32 h) {
	using namespace CityHash_Internal;

	// Helper from Murmur3 for combining two 32-bit values.
	a *= c1;
	a = Rotate32(a, 17);
	a *= c2;
	h ^= a;
	h = Rotate32(h, 19);
	return h * 5 + 0xe6546b64;
}

static uint32 Hash32Len13to24(const char *s, uint32 len) {
	uint32 a = Fetch32(s - 4 + (len >> 1));
	uint32 b = Fetch32(s + 4);
	uint32 c = Fetch32(s + len - 8);
	uint32 d = Fetch32(s + (len >> 1));
	uint32 e = Fetch32(s);
	uint32 f = Fetch32(s + len - 4);
	uint32 h = len;

	return fmix(Mur(f, Mur(e, Mur(d, Mur(c, Mur(b, Mur(a, h)))))));
}

static uint32 Hash32Len0to4(const char *s, uint32 len) {
	using namespace CityHash_Internal;

	uint32 b = 0;
	uint32 c = 9;
	for (uint32 i = 0; i < len; i++) {
		signed char v = s[i];
		b = b * c1 + v;
		c ^= b;
	}
	return fmix(Mur(b, Mur(len, c)));
}

static uint32 Hash32Len5to12(const char *s, uint32 len) {
	uint32 a = len, b = len * 5, c = 9, d = b;
	a += Fetch32(s);
	b += Fetch32(s + len - 4);
	c += Fetch32(s + ((len >> 1) & 4));
	return fmix(Mur(c, Mur(b, Mur(a, d))));
}

uint32 CityHash32(const char *s, uint32 len) {
	using namespace CityHash_Internal;

	if (len <= 24) {
		return len <= 12 ?
			(len <= 4 ? Hash32Len0to4(s, len) : Hash32Len5to12(s, len)) :
			Hash32Len13to24(s, len);
	}

	// len > 24
	uint32 h = len, g = c1 * len, f = g;
	uint32 a0 = Rotate32(Fetch32(s + len - 4) * c1, 17) * c2;
	uint32 a1 = Rotate32(Fetch32(s + len - 8) * c1, 17) * c2;
	uint32 a2 = Rotate32(Fetch32(s + len - 16) * c1, 17) * c2;
	uint32 a3 = Rotate32(Fetch32(s + len - 12) * c1, 17) * c2;
	uint32 a4 = Rotate32(Fetch32(s + len - 20) * c1, 17) * c2;
	h ^= a0;
	h = Rotate32(h, 19);
	h = h * 5 + 0xe6546b64;
	h ^= a2;
	h = Rotate32(h, 19);
	h = h * 5 + 0xe6546b64;
	g ^= a1;
	g = Rotate32(g, 19);
	g = g * 5 + 0xe6546b64;
	g ^= a3;
	g = Rotate32(g, 19);
	g = g * 5 + 0xe6546b64;
	f += a4;
	f = Rotate32(f, 19);
	f = f * 5 + 0xe6546b64;
	uint32 iters = (len - 1) / 20;
	do {
		uint32 _a0 = Rotate32(Fetch32(s) * c1, 17) * c2;
		uint32 _a1 = Fetch32(s + 4);
		uint32 _a2 = Rotate32(Fetch32(s + 8) * c1, 17) * c2;
		uint32 _a3 = Rotate32(Fetch32(s + 12) * c1, 17) * c2;
		uint32 _a4 = Fetch32(s + 16);
		h ^= _a0;
		h = Rotate32(h, 18);
		h = h * 5 + 0xe6546b64;
		f += _a1;
		f = Rotate32(f, 19);
		f = f * c1;
		g += _a2;
		g = Rotate32(g, 18);
		g = g * 5 + 0xe6546b64;
		h ^= _a3 + _a1;
		h = Rotate32(h, 19);
		h = h * 5 + 0xe6546b64;
		g ^= _a4;
		g = bswap_32(g) * 5;
		h += _a4 * 5;
		h = bswap_32(h);
		f += _a0;
		PERMUTE3(f, h, g);
		s += 20;
	} while (--iters != 0);
	g = Rotate32(g, 11) * c1;
	g = Rotate32(g, 17) * c1;
	f = Rotate32(f, 11) * c1;
	f = Rotate32(f, 17) * c1;
	h = Rotate32(h + g, 19);
	h = h * 5 + 0xe6546b64;
	h = Rotate32(h, 17) * c1;
	h = Rotate32(h + f, 19);
	h = h * 5 + 0xe6546b64;
	h = Rotate32(h, 17) * c1;
	return h;
}

// Bitwise right rotate.  Normally this will compile to a single
// instruction, especially if the shift is a manifest constant.
static uint64 Rotate(uint64 val, int shift) {
	// Avoid shifting by 64: doing so yields an undefined result.
	return shift == 0 ? val : ((val >> shift) | (val << (64 - shift)));
}

static uint64 ShiftMix(uint64 val) {
	return val ^ (val >> 47);
}

static uint64 HashLen16(uint64 u, uint64 v) {
	return CityHash128to64({ u, v });
}

static uint64 HashLen16(uint64 u, uint64 v, uint64 mul) {
	// Murmur-inspired hashing.
	uint64 a = (u ^ v) * mul;
	a ^= (a >> 47);
	uint64 b = (v ^ a) * mul;
	b ^= (b >> 47);
	b *= mul;
	return b;
}

static uint64 HashLen0to16(const char *s, uint32 len) {
	using namespace CityHash_Internal;

	if (len >= 8) {
		uint64 mul = k2 + len * 2;
		uint64 a = Fetch64(s) + k2;
		uint64 b = Fetch64(s + len - 8);
		uint64 c = Rotate(b, 37) * mul + a;
		uint64 d = (Rotate(a, 25) + b) * mul;
		return HashLen16(c, d, mul);
	}
	if (len >= 4) {
		uint64 mul = k2 + len * 2;
		uint64 a = Fetch32(s);
		return HashLen16(len + (a << 3), Fetch32(s + len - 4), mul);
	}
	if (len > 0) {
		uint8 a = s[0];
		uint8 b = s[len >> 1];
		uint8 c = s[len - 1];
		uint32 y = static_cast<uint32>(a) + (static_cast<uint32>(b) << 8);
		uint32 z = len + (static_cast<uint32>(c) << 2);
		return ShiftMix(y * k2 ^ z * k0) * k2;
	}
	return k2;
}

// This probably works well for 16-byte strings as well, but it may be overkill
// in that case.
static uint64 HashLen17to32(const char *s, uint32 len) {
	using namespace CityHash_Internal;

	uint64 mul = k2 + len * 2;
	uint64 a = Fetch64(s) * k1;
	uint64 b = Fetch64(s + 8);
	uint64 c = Fetch64(s + len - 8) * mul;
	uint64 d = Fetch64(s + len - 16) * k2;
	return HashLen16(Rotate(a + b, 43) + Rotate(c, 30) + d,
		a + Rotate(b + k2, 18) + c, mul);
}

// Return a 16-byte hash for 48 bytes.  Quick and dirty.
// Callers do best to use "random-looking" values for a and b.
static Uint128_64 WeakHashLen32WithSeeds(
	uint64 w, uint64 x, uint64 y, uint64 z, uint64 a, uint64 b) {
	a += w;
	b = Rotate(b + a + z, 21);
	uint64 c = a;
	a += x;
	a += y;
	b += Rotate(a, 44);
	return { (a + z), (b + c) };
}

// Return a 16-byte hash for s[0] ... s[31], a, and b.  Quick and dirty.
static Uint128_64 WeakHashLen32WithSeeds(
	const char* s, uint64 a, uint64 b) {
	return WeakHashLen32WithSeeds(Fetch64(s),
		Fetch64(s + 8),
		Fetch64(s + 16),
		Fetch64(s + 24),
		a,
		b);
}

// Return an 8-byte hash for 33 to 64 bytes.
static uint64 HashLen33to64(const char *s, uint32 len) {
	using namespace CityHash_Internal;

	uint64 mul = k2 + len * 2;
	uint64 a = Fetch64(s) * k2;
	uint64 b = Fetch64(s + 8);
	uint64 c = Fetch64(s + len - 24);
	uint64 d = Fetch64(s + len - 32);
	uint64 e = Fetch64(s + 16) * k2;
	uint64 f = Fetch64(s + 24) * 9;
	uint64 g = Fetch64(s + len - 8);
	uint64 h = Fetch64(s + len - 16) * mul;
	uint64 u = Rotate(a + g, 43) + (Rotate(b, 30) + c) * 9;
	uint64 v = ((a + g) ^ d) + f + 1;
	uint64 w = bswap_64((u + v) * mul) + h;
	uint64 x = Rotate(e + f, 42) + c;
	uint64 y = (bswap_64((v + w) * mul) + g) * mul;
	uint64 z = e + f + c;
	a = bswap_64((x + z) * mul + y) + b;
	b = ShiftMix((z + a) * mul + d + h) * mul;
	return b + x;
}

uint64 CityHash64(const char *s, uint32 len) {
	using namespace CityHash_Internal;

	if (len <= 32) {
		if (len <= 16) {
			return HashLen0to16(s, len);
		}
		else {
			return HashLen17to32(s, len);
		}
	}
	else if (len <= 64) {
		return HashLen33to64(s, len);
	}

	// For strings over 64 bytes we hash the end first, and then as we
	// loop we keep 56 bytes of state: v, w, x, y, and z.
	uint64 x = Fetch64(s + len - 40);
	uint64 y = Fetch64(s + len - 16) + Fetch64(s + len - 56);
	uint64 z = HashLen16(Fetch64(s + len - 48) + len, Fetch64(s + len - 24));
	Uint128_64 v = WeakHashLen32WithSeeds(s + len - 64, len, z);
	Uint128_64 w = WeakHashLen32WithSeeds(s + len - 32, y + k1, x);
	x = x * k1 + Fetch64(s);

	// Decrease len to the nearest multiple of 64, and operate on 64-byte chunks.
	len = (len - 1) & ~static_cast<uint32>(63);
	do {
		x = Rotate(x + y + v.lo + Fetch64(s + 8), 37) * k1;
		y = Rotate(y + v.hi + Fetch64(s + 48), 42) * k1;
		x ^= w.hi;
		y += v.lo + Fetch64(s + 40);
		z = Rotate(z + w.lo, 33) * k1;
		v = WeakHashLen32WithSeeds(s, v.hi * k1, x + w.lo);
		w = WeakHashLen32WithSeeds(s + 32, z + w.hi, y + Fetch64(s + 16));
		SwapValues(z, x);
		s += 64;
		len -= 64;
	} while (len != 0);
	return HashLen16(HashLen16(v.lo, w.lo) + ShiftMix(y) * k1 + z,
		HashLen16(v.hi, w.hi) + x);
}

uint64 CityHash64WithSeed(const char *s, uint32 len, uint64 seed) {
	using namespace CityHash_Internal;

	return CityHash64WithSeeds(s, len, k2, seed);
}

uint64 CityHash64WithSeeds(const char *s, uint32 len, uint64 seed0, uint64 seed1) {
	using namespace CityHash_Internal;

	return HashLen16(CityHash64(s, len) - seed0, seed1);
}
