// Copyright Epic Games, Inc. All Rights Reserved.

#include "IO/IoBuffer.h"
#include "IO/IoDispatcher.h"
#include "HAL/Platform.h"
#include "HAL/UnrealMemory.h"
#include "Misc/AssertionMacros.h"
#include "Templates/RefCounting.h"

//////////////////////////////////////////////////////////////////////////

FIoBuffer::BufCore::BufCore()
{
}

FIoBuffer::BufCore::~BufCore()
{
	if (IsMemoryOwned())
	{
		FMemory::Free(Data());
	}
}

FIoBuffer::BufCore::BufCore(const uint8* InData, uint64 InSize, bool InOwnsMemory)
{
	SetDataAndSize(InData, InSize);

	SetIsOwned(InOwnsMemory);
}

FIoBuffer::BufCore::BufCore(const uint8* InData, uint64 InSize, const BufCore* InOuter)
:	OuterCore(InOuter)
{
	SetDataAndSize(InData, InSize);
}

FIoBuffer::BufCore::BufCore(uint64 InSize)
{
	uint8* NewBuffer = reinterpret_cast<uint8*>(FMemory::Malloc(InSize));

	SetDataAndSize(NewBuffer, InSize);

	SetIsOwned(true);
}

FIoBuffer::BufCore::BufCore(ECloneTag, uint8* InData, uint64 InSize)
:	FIoBuffer::BufCore(InSize)
{
	FMemory::Memcpy(Data(), InData, InSize);
}

void
FIoBuffer::BufCore::SetDataAndSize(const uint8* InData, uint64 InSize)
{
	// This is intentionally not split into SetData and SetSize to enable different storage
	// strategies for flags in the future (in unused pointer bits)

	DataPtr			= const_cast<uint8*>(InData);
	DataSizeLow		= uint32(InSize & 0xffffffffu);
	DataSizeHigh	= (InSize >> 32) & 0xffu;
}

void
FIoBuffer::BufCore::SetSize(uint64 InSize)
{
	SetDataAndSize(Data(), InSize);
}

void
FIoBuffer::BufCore::MakeOwned()
{
	if (IsMemoryOwned())
		return;

	const uint64 BufferSize = DataSize();
	uint8* NewBuffer		= reinterpret_cast<uint8*>(FMemory::Malloc(BufferSize));

	FMemory::Memcpy(NewBuffer, Data(), BufferSize);

	SetDataAndSize(NewBuffer, BufferSize);

	SetIsOwned(true);
}

TIoStatusOr<uint8*>	FIoBuffer::BufCore::ReleaseMemory()
{
	if (IsMemoryOwned())
	{
		uint8* BufferPtr = Data();
		SetDataAndSize(nullptr, 0);
		ClearFlags();

		return BufferPtr;
	}
	else
	{
		return FIoStatus(EIoErrorCode::InvalidParameter, TEXT("Cannot call release on a FIoBuffer unless it owns it's memory"));
	}
}

//////////////////////////////////////////////////////////////////////////

FIoBuffer::FIoBuffer()
:	CorePtr(new BufCore)
{
}

FIoBuffer::FIoBuffer(uint64 InSize)
:	CorePtr(new BufCore(InSize))
{
}

FIoBuffer::FIoBuffer(const void* Data, uint64 InSize, const FIoBuffer& OuterBuffer)
:	CorePtr(new BufCore((uint8*) Data, InSize, OuterBuffer.CorePtr))
{
}

FIoBuffer::FIoBuffer(FMemoryView Memory, const FIoBuffer& OuterBuffer)
: FIoBuffer(Memory.GetData(), Memory.GetSize(), OuterBuffer)
{
}

FIoBuffer::FIoBuffer(FIoBuffer::EWrapTag, const void* Data, uint64 InSize)
:	CorePtr(new BufCore((uint8*)Data, InSize, /* ownership */ false))
{
}

FIoBuffer::FIoBuffer(FIoBuffer::EWrapTag, FMemoryView Memory)
: FIoBuffer(FIoBuffer::Wrap, Memory.GetData(), Memory.GetSize())
{
}

FIoBuffer::FIoBuffer(FIoBuffer::EAssumeOwnershipTag, const void* Data, uint64 InSize)
:	CorePtr(new BufCore((uint8*)Data, InSize, /* ownership */ true))
{
}

FIoBuffer::FIoBuffer(FIoBuffer::EAssumeOwnershipTag, FMemoryView Memory)
: FIoBuffer(FIoBuffer::AssumeOwnership, Memory.GetData(), Memory.GetSize())
{
}

FIoBuffer::FIoBuffer(FIoBuffer::ECloneTag, const void* Data, uint64 InSize)
:	CorePtr(new BufCore(Clone, (uint8*)Data, InSize))
{
}

FIoBuffer::FIoBuffer(FIoBuffer::ECloneTag, FMemoryView Memory)
: FIoBuffer(FIoBuffer::Clone, Memory.GetData(), Memory.GetSize())
{
}

void		
FIoBuffer::MakeOwned() const
{
	CorePtr->MakeOwned();
}
 
TIoStatusOr<uint8*>
FIoBuffer::Release()
{
	return CorePtr->ReleaseMemory();
}
