// Copyright Epic Games, Inc. All Rights Reserved.

#include "IO/IoChunkEncoding.h"

#include "Async/ParallelFor.h"
#include "Containers/ArrayView.h"
#include "IO/IoDispatcher.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/AES.h"
#include "Misc/Compression.h"
#include <atomic>

////////////////////////////////////////////////////////////////////////////////
bool FIoChunkEncoding::FHeader::IsValid() const
{
	return Magic == FIoChunkEncoding::ExpectedMagic
		&& RawSize < FIoChunkEncoding::MaxSize
		&& EncodedSize < FIoChunkEncoding::MaxSize;
}

uint32 FIoChunkEncoding::FHeader::GetBlockSize() const
{
	return uint32(1) << uint32(BlockSizeExponent);
}

uint32 FIoChunkEncoding::FHeader::GetBlockCount() const
{
	check(IsValid());
	return static_cast<uint32>(FMath::DivideAndRoundUp(RawSize, uint64(GetBlockSize())));
}

TConstArrayView<uint32> FIoChunkEncoding::FHeader::GetBlocks() const
{
	check(IsValid());
	const uint8* Self = reinterpret_cast<const uint8*>(this);
	return TConstArrayView<uint32>(reinterpret_cast<const uint32*>(Self + sizeof(FHeader)), GetBlockCount());
}

uint64 FIoChunkEncoding::FHeader::GetTotalHeaderSize() const
{
	check(IsValid());
	return sizeof(FHeader) + (GetBlockCount() * sizeof(uint32));
}

const FIoChunkEncoding::FHeader* FIoChunkEncoding::FHeader::Decode(FMemoryView HeaderData)
{
	if (HeaderData.GetSize() < sizeof(FHeader))
	{
		return nullptr;
	}

	const FHeader* Header = reinterpret_cast<const FHeader*>(HeaderData.GetData());
	return !Header->IsValid() || HeaderData.GetSize() < Header->GetTotalHeaderSize() ? nullptr : Header;
}

////////////////////////////////////////////////////////////////////////////////
bool FIoChunkEncoding::Encode(const FIoChunkEncodingParams& Params, FMemoryView RawData, FIoBuffer& OutEncodedData)
{
	FIoBuffer Encoding, Blocks;
	if (Encode(Params, RawData,Encoding, Blocks))
	{
		OutEncodedData = FIoBuffer(Encoding.GetSize() + Blocks.GetSize()); 
		OutEncodedData.GetMutableView().CopyFrom(Encoding.GetView());
		OutEncodedData.GetMutableView().RightChop(Encoding.GetSize()).CopyFrom(Blocks.GetView());
		return true;
	}

	return false;
}

bool FIoChunkEncoding::Encode(const FIoChunkEncodingParams& Params, FMemoryView RawData, FIoBuffer& OutHeader, FIoBuffer& OutEncodedBlocks)
{
	const uint32 BlockSize = Params.BlockSize;

	check(IsAligned(BlockSize, FAES::AESBlockSize));
	checkf(FMath::IsPowerOfTwo(BlockSize) && BlockSize <= MAX_uint32,
		TEXT("BlockSize must be a 32-bit power of two but was %u."), BlockSize);
	checkf(Params.EncryptionKey.IsEmpty() || Params.EncryptionKey.GetSize() == FAES::FAESKey::KeySize, TEXT("Encryption key must be zero or %d bytes (AES)"), FAES::FAESKey::KeySize);

	if (RawData.GetSize() == 0)
	{
		return false;
	}

	FAES::FAESKey AESKey;
	if (!Params.EncryptionKey.IsEmpty())
	{
		FMutableMemoryView(AESKey.Key, FAES::FAESKey::KeySize).CopyFrom(Params.EncryptionKey);
	}

	const int64 BlockCount = FMath::DivideAndRoundUp(RawData.GetSize(), uint64(BlockSize));
	check(BlockCount > 0);
	checkf(BlockCount <= FIoChunkEncoding::MaxBlockCount, TEXT("Raw data of size %" UINT64_FMT " with block size %u requires "
		"%" UINT64_FMT " blocks, but the limit is %d."), RawData.GetSize(), BlockSize, BlockCount, FIoChunkEncoding::MaxBlockCount);

	struct FBlock
	{
		FIoBuffer Buffer;
		uint32 Size = 0;
	};

	std::atomic_uint64_t TotalEncodedSize{0};
	TArray<FBlock> Blocks;
	Blocks.SetNum(int32(BlockCount));

	ParallelFor(int32(BlockCount),
		[BlockSize, &RawData, &Params, &AESKey, &Blocks, &TotalEncodedSize]
		(int32 BlockIndex)
		{
			const uint64 RawOffset = BlockIndex * BlockSize;
			const uint64 RawBlockSize = FMath::Min(RawData.GetSize() - RawOffset, static_cast<uint64>(BlockSize));
			FMemoryView RawBlock = RawData.Mid(RawOffset, BlockSize);

			const uint64 RequiredBlockSize =
			Align(FCompression::CompressMemoryBound(Params.CompressionFormat, static_cast<int32>(BlockSize)), FAES::AESBlockSize);

			FBlock& Block = Blocks[BlockIndex];
			Block.Buffer = FIoBuffer(RequiredBlockSize);

			int32 CompressedSize = int32(Block.Buffer.GetSize());
			const bool bCompressed = FCompression::CompressMemory(
				Params.CompressionFormat,
				Block.Buffer.GetData(),
				CompressedSize,
				RawBlock.GetData(),
				static_cast<int32>(RawBlock.GetSize()),
				COMPRESS_ForPackaging);

			if (bCompressed == false)
			{
				Block.Size = 0;
				return;
			}

			Block.Size = CompressedSize;
			if (CompressedSize >= RawBlockSize)
			{
				Block.Buffer.GetMutableView().CopyFrom(RawBlock);
				Block.Size = static_cast<uint32>(RawBlockSize);
			}

			const uint32 AESAlignedSize = Align(Block.Size, FAES::AESBlockSize);
			{
				uint8* Data = Block.Buffer.Data();
				for (uint64 FillIndex = Block.Size; FillIndex < AESAlignedSize; ++FillIndex)
				{
					check(FillIndex < Block.Buffer.GetSize());
					Data[FillIndex] = Data[(FillIndex - Block.Size) % Block.Size];
				}
			}

			if (AESKey.IsValid())
			{
				FAES::EncryptData(Block.Buffer.GetData(), AESAlignedSize, AESKey);
			}

			TotalEncodedSize.fetch_add(AESAlignedSize);
		});

	const uint64 RequiredHeaderSize = sizeof(FHeader) + (BlockCount * sizeof(uint32)); 
	if (OutHeader.GetSize() != RequiredHeaderSize)
	{
		OutHeader = FIoBuffer(RequiredHeaderSize);
	}

	const EIoEncryptionMethod Encryption = AESKey.IsValid() ? EIoEncryptionMethod::AES : EIoEncryptionMethod::None;

	FHeader* Header = reinterpret_cast<FHeader*>(OutHeader.GetData());
	Header->Magic = ExpectedMagic;
	Header->RawSize = RawData.GetSize();
	Header->EncodedSize = TotalEncodedSize;
	Header->BlockSizeExponent = uint8(FMath::FloorLog2(BlockSize));
	Header->Flags = static_cast<uint8>(Encryption); 
	Header->Pad = 0;

	TArrayView<uint32> BlockSizes = MakeArrayView(
		reinterpret_cast<uint32*>(OutHeader.GetMutableView().RightChop(sizeof(FHeader)).GetData()),
		int32(BlockCount));

	OutEncodedBlocks = FIoBuffer(TotalEncodedSize);
	FMutableMemoryView EncodedBlocks = OutEncodedBlocks.GetMutableView();

	for (int32 BlockIndex = 0; BlockIndex < BlockCount; BlockIndex++)
	{
		FBlock& Block = Blocks[BlockIndex];
		if (Block.Size == 0)
		{
			return false;
		}

		const uint32 AlignedBlockSize = Align(Block.Size, FAES::AESBlockSize);
		BlockSizes[BlockIndex] = AlignedBlockSize;
		EncodedBlocks.CopyFrom(Block.Buffer.GetView().Left(AlignedBlockSize));
		EncodedBlocks += AlignedBlockSize;
	}

	check(EncodedBlocks.GetSize() == 0);
	check(FHeader::Decode(OutHeader.GetView()) != nullptr);

	return true;
}

////////////////////////////////////////////////////////////////////////////////
bool FIoChunkEncoding::Decode(
	FMemoryView EncodedData,
	FName CompressionFormat,
	FMemoryView EncryptionKey,
	FMutableMemoryView OutRawData,
	uint64 RawOffset)
{
	const FHeader* Header = FHeader::Decode(EncodedData);
	if (Header == nullptr)
	{
		return false;
	}

	const uint32 BlockCount = Header->GetBlockCount();

	FIoChunkDecodingParams Params;
	Params.CompressionFormat = CompressionFormat;
	Params.EncryptionKey = EncryptionKey;
	Params.RawOffset = RawOffset;
	Params.BlockSize = Header->GetBlockSize();
	Params.TotalRawSize = Header->RawSize;
	Params.EncodedBlockSize = Header->GetBlocks(); 

	const uint64 TotalHeaderSize = sizeof(FHeader) + (BlockCount * sizeof(uint32));
	FMemoryView EncodedBlocks = EncodedData.RightChop(TotalHeaderSize);

	return Decode(Params, EncodedBlocks, OutRawData);
}

bool FIoChunkEncoding::Decode(const FIoChunkDecodingParams& Params, FMemoryView EncodedBlocks, FMutableMemoryView OutRawData, EIoDecodeFlags Flags)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FIoChunkEncoding::Decode);

	if (Params.TotalRawSize < Params.RawOffset + OutRawData.GetSize())
	{
		return false;
	}

	checkf(Params.EncryptionKey.IsEmpty() || Params.EncryptionKey.GetSize() == FAES::FAESKey::KeySize, TEXT("Encryption key must be %d bytes (AES)"), FAES::FAESKey::KeySize);
	FAES::FAESKey AESKey;
	if (!Params.EncryptionKey.IsEmpty())
	{
		FMutableMemoryView(AESKey.Key, FAES::FAESKey::KeySize).CopyFrom(Params.EncryptionKey);
	}

	const TConstArrayView<uint32>& EncodedBlockSize = Params.EncodedBlockSize;
	const TConstArrayView<uint32>& BlockHash = Params.BlockHash;
	const uint32 BlockSize = Params.BlockSize;
	const uint64 RawOffset = Params.RawOffset;
	const uint32 BlockCount = EncodedBlockSize.Num();
	const uint64 LastRawBlockSize = BlockSize - (BlockSize * BlockCount - Params.TotalRawSize);
	const uint32 FirstBlockIndex = uint32(RawOffset / BlockSize);
	const uint32 LastBlockIndex = uint32((RawOffset + OutRawData.GetSize() - 1) / BlockSize);

	uint64 RawBlockOffset = RawOffset % BlockSize;

	uint64 EncodedOffset = 0;
	for (uint32 BlockIndex = 0; BlockIndex < FirstBlockIndex; ++BlockIndex)
	{
		EncodedOffset += Align(EncodedBlockSize[BlockIndex], FAES::AESBlockSize);
	}
	// Subtract the encoded offset if the encoded blocks is a partial range of all encoded block(s)
	EncodedBlocks += (EncodedOffset - Params.EncodedOffset);

	// Buffer used to decrypt blocks into, only allocated if a valid ASE key is found
	TUniquePtr<uint8[]> DecryptedBlockBuffer;
	uint64 DecryptedScratchBufferSize = 0;

	if (AESKey.IsValid())
	{
		for (uint32 BlockIndex = FirstBlockIndex; BlockIndex <= LastBlockIndex; BlockIndex++)
		{
			const uint64 AlignedBlockSize = Align(EncodedBlockSize[BlockIndex], FAES::AESBlockSize);
			DecryptedScratchBufferSize = FMath::Max(AlignedBlockSize, DecryptedScratchBufferSize);
		}

		DecryptedBlockBuffer = MakeUnique<uint8[]>(DecryptedScratchBufferSize);
	}

	const bool bForceTempBuffer = EnumHasAnyFlags(Flags, EIoDecodeFlags::WriteOnly);
	const bool bVerifyBlocks = BlockHash.IsEmpty() == false;

	for (uint32 BlockIndex = FirstBlockIndex; BlockIndex <= LastBlockIndex; BlockIndex++)
	{
		const uint64 RawBlockSize = BlockIndex == BlockCount - 1 ? LastRawBlockSize : BlockSize;
		const uint64 RawBlockReadSize = FMath::Min(OutRawData.GetSize(), RawBlockSize - RawBlockOffset);
		const uint32 CompressedBlockSize = EncodedBlockSize[BlockIndex];
		const uint32 AlignedBlockSize = Align(CompressedBlockSize, FAES::AESBlockSize);

		FMemoryView BlockView = EncodedBlocks.Left(AlignedBlockSize);

		if (bVerifyBlocks)
		{
			check(BlockHash.IsEmpty() == false);
			FIoBlockHash Hash = HashBlock(BlockView);
			if (Hash != BlockHash[BlockIndex])
			{
				UE_LOGF(LogIoDispatcher, Warning, "Block hash mismatch, index '%d', hash '0x%X', expected hash '0x%X'",
					BlockIndex, Hash, BlockHash[BlockIndex]);
				return false;
			}
		}

		if (AESKey.IsValid())
		{
			check(BlockView.GetSize() <= DecryptedScratchBufferSize);
			FMemory::Memcpy(DecryptedBlockBuffer.Get(), BlockView.GetData(), BlockView.GetSize());
			
			FAES::DecryptData(DecryptedBlockBuffer.Get(), BlockView.GetSize(), AESKey);

			BlockView = MakeMemoryView(DecryptedBlockBuffer.Get(), BlockView.GetSize());
		}

		if (CompressedBlockSize < RawBlockSize)
		{
			if (RawBlockReadSize == RawBlockSize && !bForceTempBuffer)
			{
				if (!FCompression::UncompressMemory(Params.CompressionFormat, OutRawData.GetData(), int32(RawBlockReadSize), BlockView.GetData(), CompressedBlockSize))
				{
					UE_LOG(LogIoDispatcher, Warning, TEXT("Failed to uncompress block '%d', format '%s', raw size '%" UINT64_FMT "' compressed size '%u'"),
						BlockIndex, *Params.CompressionFormat.ToString(), RawBlockSize, AlignedBlockSize);
					return false;
				}
			}
			else
			{
				FIoBuffer RawBlockTmp = FIoBuffer(RawBlockSize);
				if (!FCompression::UncompressMemory(Params.CompressionFormat, RawBlockTmp.GetData(), int32(RawBlockSize), BlockView.GetData(), CompressedBlockSize))
				{
					UE_LOG(LogIoDispatcher, Warning, TEXT("Failed to uncompress block '%d', format '%s', raw size '%" UINT64_FMT "' compressed size '%u'"),
						BlockIndex, *Params.CompressionFormat.ToString(), RawBlockSize, AlignedBlockSize);
					return false;
				}
				OutRawData.CopyFrom(RawBlockTmp.GetView().Mid(RawBlockOffset, RawBlockReadSize));
			}
		}
		else
		{
			OutRawData.CopyFrom(BlockView.Mid(RawBlockOffset, RawBlockReadSize));
		}

		RawBlockOffset = 0;
		OutRawData += RawBlockReadSize;
		EncodedBlocks += AlignedBlockSize;
	}

	check(OutRawData.GetSize() == 0);

	return true;
}

FIoChunkBlockDecodeResult FIoChunkEncoding::DecodeBlock(const FIoChunkBlockDecodingParams Params, FMutableMemoryView EncodedBlock, FMutableMemoryView DecodedBlock)
{
	// Assume that the underlying memory size is aligned to AES block size 
	const uint32 AlignedEncodedSize = Align(uint32(EncodedBlock.GetSize()), FAES::AESBlockSize);

	if (Params.BlockHash.IsEmpty() == false)
	{
		check(Params.HashFunction == EIoSignatureHashFunction::Sha1);
		FSHAHash ShaHash;
		FSHA1::HashBuffer(EncodedBlock.GetData(), AlignedEncodedSize, ShaHash.Hash);
		FMemoryView BlockHash(ShaHash.Hash, sizeof(ShaHash.Hash));

		if (0 != Params.BlockHash.CompareBytes(BlockHash))
		{
			return FIoChunkBlockDecodeResult
			{
				.ErrorCode = EIoErrorCode::SignatureError
			};
		}
	}

	if (Params.EncryptionKey.IsEmpty() == false)
	{
		FAES::FAESKey AESKey;
		FMutableMemoryView(AESKey.Key, FAES::FAESKey::KeySize).CopyFrom(Params.EncryptionKey);
		FAES::DecryptData((uint8*)EncodedBlock.GetData(), AlignedEncodedSize, AESKey);
	}

	if (Params.CompressionFormat.IsNone())
	{
		return FIoChunkBlockDecodeResult
		{
			.DecodedBlock	= FMutableMemoryView(EncodedBlock.GetData(), DecodedBlock.GetSize()),
			.ErrorCode		= EIoErrorCode::Ok
		};
	}

	if (LIKELY(FCompression::UncompressMemory(
		Params.CompressionFormat,
		DecodedBlock.GetData(),
		IntCastChecked<int32>(DecodedBlock.GetSize()),
		EncodedBlock.GetData(),
		IntCastChecked<int32>(EncodedBlock.GetSize()))))
	{
		return FIoChunkBlockDecodeResult
		{
			.DecodedBlock	= FMutableMemoryView(DecodedBlock.GetData(), DecodedBlock.GetSize()),
			.ErrorCode		= EIoErrorCode::Ok
		};
	}

	return FIoChunkBlockDecodeResult
	{
		.ErrorCode = EIoErrorCode::CompressionError
	};
}

////////////////////////////////////////////////////////////////////////////////
TIoStatusOr<FIoOffsetAndLength> FIoChunkEncoding::GetChunkRange(uint64 TotalRawSize, uint32 RawBlockSize, TConstArrayView<uint32> EncodedBlockSize, uint64 RawOffset, uint64 RawSize)
{
	check(RawBlockSize > 0);
	check(EncodedBlockSize.Num() > 0);

	if (TotalRawSize < RawOffset + RawSize)
	{
		return FIoStatus(EIoErrorCode::InvalidParameter);
	}

	const uint64 TotalBlockCount = FMath::DivideAndRoundUp(TotalRawSize, uint64(RawBlockSize));
	const uint32 FirstBlockIndex = uint32(RawOffset / RawBlockSize);
	const uint32 LastBlockIndex = uint32((RawOffset + RawSize - 1) / RawBlockSize);

	if (TotalBlockCount < FirstBlockIndex || TotalBlockCount < LastBlockIndex)
	{
		return FIoStatus(EIoErrorCode::InvalidParameter);
	}

	uint64 Offset = 0;
	uint32 BlockIndex = 0;
	while (BlockIndex < FirstBlockIndex)
	{
		Offset += Align(EncodedBlockSize[BlockIndex++], FAES::AESBlockSize);
	}

	uint64 Length = 0;
	while (BlockIndex <= LastBlockIndex)
	{
		Length += Align(EncodedBlockSize[BlockIndex++], FAES::AESBlockSize);
	}

	return FIoOffsetAndLength(Offset, Length);
}

TIoStatusOr<FIoOffsetAndLength> FIoChunkEncoding::GetChunkRange(const FIoChunkDecodingParams& Params, uint64 RawSize)
{
	return GetChunkRange(Params.TotalRawSize, Params.BlockSize, Params.EncodedBlockSize, Params.RawOffset, RawSize);
}

uint64 FIoChunkEncoding::GetTotalEncodedSize(TConstArrayView<uint32> EncodedBlockSize)
{
	uint64 TotalEncodedSize = 0;
	for (uint64 BlockSize : EncodedBlockSize)
	{
		TotalEncodedSize += Align(BlockSize, FAES::AESBlockSize);
	}

	return TotalEncodedSize;
}

FIoBlockHash FIoChunkEncoding::HashBlock(FMemoryView Block)
{
	const FIoHash Hash = FIoHash::HashBuffer(Block);
	FIoBlockHash BlockHash;
	FMemory::Memcpy(&BlockHash, &Hash, sizeof(FIoBlockHash));
	return BlockHash;
}

////////////////////////////////////////////////////////////////////////////////
void FIoChunkBlockDecoder::Set(IIoChunkBlockDecoder& InInstance)
{
	check(Instance == nullptr);
	Instance = &InInstance;
}

IIoChunkBlockDecoder& FIoChunkBlockDecoder::Get()
{
	check(Instance);
	return *Instance;
}

void FIoChunkBlockDecoder::Release()
{
	Instance = nullptr;
}

IIoChunkBlockDecoder* FIoChunkBlockDecoder::Instance = nullptr;
