// Copyright Epic Games, Inc. All Rights Reserved.

#include "IO/IoChunkId.h"
#include "IO/PackageId.h"
#include "IO/IoContainerId.h"
#include "Hash/Blake3.h"
#include "Memory/MemoryView.h"
#include "Serialization/Archive.h"
#include "Serialization/CompactBinaryWriter.h"
#include "String/HexToBytes.h"

FString LexToString(const EIoChunkType Type)
{
	const TCHAR* Strings[] = {
		TEXT("Invalid"),
		TEXT("ExportBundleData"),
		TEXT("BulkData"),
		TEXT("OptionalBulkData"),
		TEXT("MemoryMappedBulkData"),
		TEXT("ScriptObjects"),
		TEXT("ContainerHeader"),
		TEXT("ExternalFile"),
		TEXT("ShaderCodeLibrary"),
		TEXT("ShaderCode"),
		TEXT("PackageStoreEntry"),
		TEXT("DerivedData"),
		TEXT("EditorDerivedData"),
		TEXT("PackageResource"),
	};
	static_assert(UE_ARRAY_COUNT(Strings) == (SIZE_T)EIoChunkType::MAX);
	
	const uint8 Index = (uint8)Type;
	if (Index < UE_ARRAY_COUNT(Strings))
	{
		return Strings[Index];
	}
	else
	{
		return Strings[0]; // return Invalid
	}
}

FString LexToString(const FIoChunkId& Id)
{
	FString Output;
	Id.ToString(Output);
	return Output;
}

void FIoChunkId::ToString(FString& Output) const
{
	Output.Reset();
	TArray<TCHAR, FString::AllocatorType>& CharArray = Output.GetCharArray();
	CharArray.AddUninitialized(sizeof(FIoChunkId) * 2 + 1);
	UE::String::BytesToHexLower(Id, CharArray.GetData());
	CharArray.Last() = TCHAR('\0');
}

FIoChunkId FIoChunkId::FromHex(FStringView Hex)
{
	if (Hex.Len() == sizeof(FIoChunkId) * 2)
	{
		FIoChunkId ChunkId;
		UE::String::HexToBytes(Hex, ChunkId.Id);
		return ChunkId;
	}

	return InvalidChunkId;
}

FArchive& operator<<(FArchive& Ar, FIoChunkId& ChunkId)
{
	Ar.Serialize(&ChunkId.Id, sizeof FIoChunkId::Id);
	return Ar;
}

FCbWriter& operator<<(FCbWriter& Writer, const FIoChunkId& ChunkId)
{
	Writer.AddObjectId(FCbObjectId(MakeMemoryView(&ChunkId.Id, sizeof FIoChunkId::Id)));
	return Writer;
}

bool LoadFromCompactBinary(FCbFieldView Field, FIoChunkId& OutChunkId)
{
	FCbObjectId ObjectId = Field.AsObjectId();
	OutChunkId.Set(ObjectId.GetView());
	return !Field.HasError();
}

FIoChunkId CreatePackageDataChunkId(const FPackageId& PackageId)
{
	return CreateIoChunkId(PackageId.Value(), 0, EIoChunkType::ExportBundleData);
}

FIoChunkId CreateExternalFileChunkId(const FStringView Filename)
{
	check(Filename.Len() > 0);

	TArray<TCHAR, TInlineAllocator<FName::StringBufferSize>> Buffer;
	Buffer.SetNum(Filename.Len());

	for (int32 Idx = 0, Len = Filename.Len(); Idx < Len; Idx++)
	{
		TCHAR Char = TChar<TCHAR>::ToLower(Filename[Idx]);
		if (Char == TEXT('\\'))
		{
			Char = TEXT('/');
		}

		Buffer[Idx] = Char;
	}

	const FBlake3Hash Hash = FBlake3::HashBuffer(FMemoryView(Buffer.GetData(), Buffer.Num() * Buffer.GetTypeSize()));

	uint8 Id[12] = {0};
	FMemory::Memcpy(Id, Hash.GetBytes(), 11);
	Id[11] = uint8(EIoChunkType::ExternalFile);

	FIoChunkId ChunkId;
	ChunkId.Set(Id, sizeof(Id));

	return ChunkId;
}

FIoChunkId CreateContainerHeaderChunkId(const FIoContainerId& ContainerId)
{
	return CreateIoChunkId(ContainerId.Value(), 0, EIoChunkType::ContainerHeader);
}
