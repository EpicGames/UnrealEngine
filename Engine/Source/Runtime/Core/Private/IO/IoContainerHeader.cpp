// Copyright Epic Games, Inc. All Rights Reserved.

#include "IO/IoContainerHeader.h"
#include "UObject/NameBatchSerialization.h"

FArchive& operator<<(FArchive& Ar, FIoContainerHeaderPackageRedirect& Redirect)
{
	Ar << Redirect.SourcePackageId;
	Ar << Redirect.TargetPackageId;
	Ar << Redirect.SourcePackageName;

	return Ar;
}

FArchive& operator<<(FArchive& Ar, FIoContainerHeaderLocalizedPackage& LocalizedPackage)
{
	Ar << LocalizedPackage.SourcePackageId;
	Ar << LocalizedPackage.SourcePackageName;

	return Ar;
}

void FIoContainerHeaderSoftPackageReferences::Empty()
{
	PackageIds.Empty();
	PackageIndices.Empty();
	bContainsSoftPackageReferences = false;
}

FArchive& operator<<(FArchive& Ar, FIoContainerHeaderSoftPackageReferences& SoftPackageReferences)
{
	Ar << SoftPackageReferences.bContainsSoftPackageReferences;
	if (SoftPackageReferences.bContainsSoftPackageReferences)
	{
		Ar << SoftPackageReferences.PackageIds;
		Ar << SoftPackageReferences.PackageIndices;
		SoftPackageReferences.bLoadedSoftPackageReferences = true;
	}
	return Ar;
}

FArchive& operator<<(FArchive& Ar, FIoContainerHeaderSerialInfo& SerialInfo)
{
	Ar << SerialInfo.Offset;
	Ar << SerialInfo.Size;
	return Ar;
}

FArchive& operator<<(FArchive& Ar, FIoContainerHeader& ContainerHeader)
{
	uint32 Signature = FIoContainerHeader::Signature;
	Ar << Signature;
	if (Ar.IsLoading() && Signature != FIoContainerHeader::Signature)
	{
		Ar.SetError();
		return Ar;
	}
	EIoContainerHeaderVersion Version = EIoContainerHeaderVersion::Latest;
	Ar << Version;
	if (Ar.IsLoading() && Version < EIoContainerHeaderVersion::NoExportInfo)
	{
		// This version is too old
		Ar.SetError();
		return Ar;
	}
	Ar << ContainerHeader.ContainerId;
	Ar << ContainerHeader.PackageIds;
	Ar << ContainerHeader.StoreEntries;
	Ar << ContainerHeader.OptionalSegmentPackageIds;
	Ar << ContainerHeader.OptionalSegmentStoreEntries;
	if (Ar.IsLoading())
	{
		ContainerHeader.RedirectsNameMap = LoadNameBatch(Ar);
	}
	else
	{
#if ALLOW_NAME_BATCH_SAVING
		SaveNameBatch(ContainerHeader.RedirectsNameMap, Ar);
#else
		check(false);
#endif
	}
	Ar << ContainerHeader.LocalizedPackages;
	Ar << ContainerHeader.PackageRedirects;

	if (Version == EIoContainerHeaderVersion::SoftPackageReferences)
	{
		Ar << ContainerHeader.SoftPackageReferences;
	}
	else if (Version >= EIoContainerHeaderVersion::SoftPackageReferencesOffset)
	{
		const int64 SerialInfoOffset = Ar.Tell();
		Ar << ContainerHeader.SoftPackageReferencesSerialInfo;
		if (Ar.IsLoading())
		{
			if (ContainerHeader.SoftPackageReferencesSerialInfo.Size > 0)
			{
				const int64 EndPos = Ar.Tell() + ContainerHeader.SoftPackageReferencesSerialInfo.Size;
				if (EndPos > Ar.TotalSize())
				{
					Ar.SetError();
					return Ar;
				}
				Ar.Seek(EndPos);
			}
		}
		else 
		{
			ContainerHeader.SoftPackageReferencesSerialInfo.Offset = Ar.Tell();
			Ar << ContainerHeader.SoftPackageReferences;
			ContainerHeader.SoftPackageReferencesSerialInfo.Size = Ar.Tell() - ContainerHeader.SoftPackageReferencesSerialInfo.Offset;
			FArchive::FScopeSeekTo _(Ar, SerialInfoOffset);
			Ar << ContainerHeader.SoftPackageReferencesSerialInfo;
		}
	}

	return Ar;
}
