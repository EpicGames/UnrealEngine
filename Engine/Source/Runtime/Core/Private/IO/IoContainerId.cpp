// Copyright Epic Games, Inc. All Rights Reserved.

#include "IO/IoContainerId.h"

#include "Hash/CityHash.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Char.h"
#include "Serialization/Archive.h"
#include "Serialization/CompactBinaryWriter.h"
#include "Serialization/CompactBinarySerialization.h"
#include "Serialization/StructuredArchiveSlots.h"

FIoContainerId FIoContainerId::FromName(const FName& Name)
{
	TCHAR NameStr[FName::StringBufferSize];
	uint32 NameLen = Name.ToString(NameStr);
	for (uint32 I = 0; I < NameLen; ++I)
	{
		NameStr[I] = TChar<TCHAR>::ToLower(NameStr[I]);
	}
	uint64 Hash = CityHash64(reinterpret_cast<const char*>(NameStr), NameLen * sizeof(TCHAR));
	checkf(Hash != InvalidId, TEXT("Container name hash collision \"%s\" and InvalidId"), NameStr);
	return FIoContainerId(Hash);
}

FArchive& operator<<(FArchive& Ar, FIoContainerId& ContainerId)
{
	Ar << ContainerId.Id;

	return Ar;
}

void operator<<(FStructuredArchiveSlot Slot, FIoContainerId& ContainerId)
{
	Slot << ContainerId.Id;
}

FCbWriter& operator<<(FCbWriter& Writer, const FIoContainerId& ContainerId)
{
	Writer << ContainerId.Id;
	return Writer;
}

FString LexToString(const FIoContainerId& ContainerId)
{
	return FString::Printf(TEXT("%llx"), ContainerId.Id);
}

bool LoadFromCompactBinary(FCbFieldView Field, FIoContainerId& OutContainerId)
{
	if (Field.IsInteger())
	{
		OutContainerId.Id = Field.AsUInt64();
		return true;
	}

	return false;
}
