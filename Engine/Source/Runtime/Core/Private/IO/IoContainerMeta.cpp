// Copyright Epic Games, Inc. All Rights Reserved.

#include "IO/IoContainerMeta.h"

#include "Algo/BinarySearch.h"
#include "Async/UniqueLock.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/PathViews.h"
#include "Serialization/Archive.h"
#include "Serialization/LargeMemoryWriter.h"

////////////////////////////////////////////////////////////////////////////////
namespace IoContainerMeta
{

static bool IsValid(uint32 Index)
{
	return Index != ~uint32(0);
}

static FStringView GetNextDirectoryName(const FStringView& Path)
{
	int32 SeparatorIndex = INDEX_NONE;
	if (Path.FindChar(TEXT('/'), SeparatorIndex))
	{
		return Path.Left(SeparatorIndex);
	}

	return FStringView();
}

} // namespace IoContainerMeta

////////////////////////////////////////////////////////////////////////////////
FArchive& operator<<(FArchive& Ar, FIoMetaDirectoryIndexEntry& Entry)
{
	Ar << Entry.Name;
	Ar << Entry.ParentEntry;
	Ar << Entry.FirstChildEntry;
	Ar << Entry.NextSiblingEntry;
	Ar << Entry.FirstFileEntry;

	return Ar;
}

////////////////////////////////////////////////////////////////////////////////
FArchive& operator<<(FArchive& Ar, FIoMetaFileIndexEntry& Entry)
{
	Ar << Entry.Name;
	Ar << Entry.ContainerName;
	Ar << Entry.DirectoryEntry;
	Ar << Entry.NextFileEntry;

	return Ar;
}

////////////////////////////////////////////////////////////////////////////////
bool FIoContainerMetaHeader::IsValid() const
{
	if (FMemory::Memcmp(Magic, MagicSequence, sizeof(MagicSequence)) != 0)
	{
		return false;
	}

	return static_cast<EVersion>(Version) != EVersion::Invalid;
}

const TCHAR* FIoContainerMetaHeader::FileExtension = TEXT(".umeta");

////////////////////////////////////////////////////////////////////////////////
FIoContainerMetaReader::~FIoContainerMetaReader()
{
}

FUtf8StringView	FIoContainerMetaReader::GetFilename(const FIoChunkId& ChunkId, FUtf8StringBuilderBase& OutFilename, FUtf8StringView& OutContainerName)
{
	if (int32 Index = Algo::LowerBound(View.ChunkIds, ChunkId); Index < View.ChunkIds.Num())
	{
		if (View.ChunkIds[Index] == ChunkId)
		{
			const uint32 File = View.FileEntryIndices[Index];
			return GetFilename(File, OutFilename, OutContainerName);
		}
	}

	OutFilename.Reset();
	return OutContainerName = FUtf8StringView();
}

void FIoContainerMetaReader::Iterate(FIoContainerMetaVisistor&& Visitor)
{
	TUtf8StringBuilder<2048> Sb;
	for (int32 Idx = 0; const FIoChunkId& ChunkId : View.ChunkIds)
	{
		const uint32	File = View.FileEntryIndices[Idx++];
		FUtf8StringView ContainerName;
		FUtf8StringView Filename = GetFilename(File, Sb, ContainerName);

		if (Visitor(ChunkId, ContainerName, Filename) == false)
		{
			break;
		}
	}
}

TIoStatusOr<FIoContainerMetaReader> FIoContainerMetaReader::Load(const FString& FilePath, bool bEnableMemoryMapping)
{
	IPlatformFile& Ipf = FPlatformFileManager::Get().GetPlatformFile();

	const int64 FileSize = Ipf.FileSize(*FilePath);
	if (FileSize < int64(sizeof(FIoContainerMetaHeader)) || FileSize >= MAX_int32)
	{
		return FIoStatus(EIoErrorCode::FileNotOpen);
	}

	if (FPlatformProperties::SupportsMemoryMappedFiles() && bEnableMemoryMapping)
	{
		FOpenMappedResult OpenResult = Ipf.OpenMappedEx(*FilePath);
		if (OpenResult.HasError() == false)
		{
			FMappedFileStorage FileStorage;
			FileStorage.MappedFile		= OpenResult.StealValue();
			FileStorage.MappedRegion	= TUniquePtr<IMappedFileRegion>(FileStorage.MappedFile->MapRegion());

			FIoContainerMetaResourceView View = FIoContainerMetaResourceView::MakeView(
				FMemoryView(FileStorage.MappedRegion->GetMappedPtr(), FileStorage.MappedRegion->GetMappedSize()));

			if (View.IsValid())
			{
				return FIoContainerMetaReader(MoveTemp(FileStorage), View);
			}
		}
	}

	TUniquePtr<IFileHandle> FileHandle(Ipf.OpenRead(*FilePath));
	if (FileHandle.IsValid() == false)
	{
		return FIoStatus(EIoErrorCode::FileOpenFailed);
	}

	FMemoryStorage MemoryStorage;
	MemoryStorage.Buffer.SetNumUninitialized(int32(FileSize));

	if (FileHandle->Read(MemoryStorage.Buffer.GetData(), FileSize) == false)
	{
		return FIoStatus(EIoErrorCode::ReadError);
	}

	FIoContainerMetaResourceView View = FIoContainerMetaResourceView::MakeView(
		FMemoryView(MemoryStorage.Buffer.GetData(), MemoryStorage.Buffer.Num()));

	if (View.IsValid() == false)
	{
		return FIoStatus(EIoErrorCode::FileCorrupt);
	}

	return FIoContainerMetaReader(MoveTemp(MemoryStorage), View);
}

FUtf8StringView	FIoContainerMetaReader::GetFilename(uint32 File, FUtf8StringBuilderBase& OutFilename, FUtf8StringView& OutContainerName)
{
	check(File != ~uint32(0));

	OutFilename.Reset();

	const FIoMetaFileIndexEntry&			FileEntry = View.FileEntries[File];
	uint32									Directory = FileEntry.DirectoryEntry;
	TArray<uint32, TInlineAllocator<32>>	Segments;

	check(FileEntry.Name != ~uint32(0));
	check(Directory != ~uint32(0));
	Segments.Add(FileEntry.Name);

	while (Directory != ~uint32(0) && Directory != 0)
	{
		const FIoMetaDirectoryIndexEntry& DirectoryEntry = View.DirectoryEntries[Directory];
		check(DirectoryEntry.Name != ~uint32(0));
		Segments.Add(DirectoryEntry.Name);
		Directory = DirectoryEntry.ParentEntry;
	}

	for (int32 Seg = Segments.Num() - 1; Seg >= 0; --Seg)
	{
		FUtf8StringView String = View.GetString(Segments[Seg]);

		OutFilename << FString(String);
		if (Seg > 0)
		{
			OutFilename << TEXT("/");
		}
	}

	OutContainerName = View.GetString(FileEntry.ContainerName);
	return OutFilename.ToView();
}

////////////////////////////////////////////////////////////////////////////////
FIoContainerMetaWriter::FIoContainerMetaWriter()
{
	// Root directory
	DirectoryEntries.AddDefaulted();
}

void FIoContainerMetaWriter::AddFile(FStringView ContainerName, const FIoChunkId& ChunkId, FStringView Filename)
{
	UE::TUniqueLock Lock(Mutex);

	// Only store filenames for packages for now
	if (ChunkIdToFileEntry.Contains(ChunkId) || ChunkId.GetChunkType() != EIoChunkType::ExportBundleData)
	{
		return;
	}

	FStringView		CleanFilename = FPathViews::GetCleanFilename(Filename);
	const uint32	Directory = CreateDirectoryTree(Filename);
	const uint32	FileEntry = AddFile(CleanFilename, ContainerName, Directory);

	ChunkIdToFileEntry.Add(ChunkId, FileEntry);
}

void FIoContainerMetaWriter::Save(FArchive& Ar)
{
	UE::TUniqueLock Lock(Mutex);

	check(ChunkIdToFileEntry.Num() == FileEntries.Num());

	TArray<FIoChunkId>	ChunkIds;
	TArray<uint32>		FileEntryIndices;

	ChunkIdToFileEntry.GetKeys(ChunkIds);
	ChunkIds.Sort();
	FileEntryIndices.Reserve(ChunkIds.Num());

	for (const FIoChunkId& ChunkId : ChunkIds)
	{
		FileEntryIndices.Add(ChunkIdToFileEntry.FindRef(ChunkId));
	}

	check(ChunkIds.Num() == FileEntryIndices.Num());
	check(FileEntryIndices.Num() == FileEntries.Num());

	FIoContainerMetaHeader Header;
	Header.Version			= static_cast<uint32>(FIoContainerMetaHeader::EVersion::Latest);
	Header.FileCount		= ChunkIds.Num();
	Header.DirectoryCount	= DirectoryEntries.Num();
	Header.StringCount		= Strings.Num();
	FMemory::Memcpy(Header.Magic, FIoContainerMetaHeader::MagicSequence, sizeof(FIoContainerMetaHeader::MagicSequence));

	Ar.Serialize(&Header, sizeof(FIoContainerMetaHeader));
	Ar.Serialize(ChunkIds.GetData(), ChunkIds.GetTypeSize() * ChunkIds.Num());
	Ar.Serialize(FileEntryIndices.GetData(), FileEntryIndices.GetTypeSize() * FileEntryIndices.Num());
	Ar.Serialize(FileEntries.GetData(), FileEntries.GetTypeSize() * FileEntries.Num());
	Ar.Serialize(DirectoryEntries.GetData(), DirectoryEntries.GetTypeSize() * DirectoryEntries.Num());
	
	{
		FLargeMemoryWriter	StringTableEntryAr;
		FLargeMemoryWriter	StringsAr;

		for (const FUtf8String& String : Strings)
		{
			uint32 Offset	= static_cast<uint32>(StringsAr.Tell());
			uint32 Len		= String.Len();

			StringTableEntryAr << Offset;
			StringTableEntryAr << Len;

			StringsAr.Serialize((void*)*String, sizeof(UTF8CHAR) * String.Len());
		}

		Ar.Serialize(StringTableEntryAr.GetData(), StringTableEntryAr.TotalSize());
		Ar.Serialize(StringsAr.GetData(), StringsAr.TotalSize());
	}

	StringToIndex.Empty();
	ChunkIdToFileEntry.Empty();
	Strings.Empty();
	DirectoryEntries = TArray<FIoMetaDirectoryIndexEntry>();
	DirectoryEntries.AddDefaulted();
	FileEntries = TArray<FIoMetaFileIndexEntry>();
}

int64 FIoContainerMetaWriter::Save(const FString& FilePath)
{
	if (TUniquePtr<FArchive> Ar(IFileManager::Get().CreateFileWriter(*FilePath)); Ar.IsValid())
	{
		Save(*Ar);
		if (int64 FileSize = Ar->TotalSize(); Ar->Close())
		{
			return FileSize;
		}
	}

	return -1;
}

uint32 FIoContainerMetaWriter::GetDirectory(uint32 DirectoryName, uint32 Parent)
{
	check(DirectoryEntries.IsValidIndex(int32(Parent)));
	uint32 Directory = DirectoryEntries[Parent].FirstChildEntry;
	while (IoContainerMeta::IsValid(Directory))
	{
		const FIoMetaDirectoryIndexEntry& Entry = DirectoryEntries[Directory];
		if (Entry.Name == DirectoryName)
		{
			return Directory;
		}
		Directory = Entry.NextSiblingEntry;
	}

	return ~uint32(0);
}

uint32 FIoContainerMetaWriter::CreateDirectory(FStringView DirectoryName, uint32 Parent)
{
	check(DirectoryEntries.IsValidIndex(int32(Parent)));

	uint32 Name = GetNameIndex(DirectoryName);
	uint32 Directory = GetDirectory(Name, Parent);

	if (IoContainerMeta::IsValid(Directory))
	{
		return Directory;
	}

	Directory = DirectoryEntries.Num();
	FIoMetaDirectoryIndexEntry& NewEntry = DirectoryEntries.AddDefaulted_GetRef();
	NewEntry.Name = Name;
	NewEntry.ParentEntry = Parent;
	NewEntry.NextSiblingEntry = DirectoryEntries[Parent].FirstChildEntry;
	DirectoryEntries[Parent].FirstChildEntry = Directory;

	return Directory;
}

uint32 FIoContainerMetaWriter::CreateDirectoryTree(FStringView Path)
{
	using namespace IoContainerMeta;

	uint32		Directory = 0; // Root
	FStringView DirectoryName = GetNextDirectoryName(Path);

	while (DirectoryName.IsEmpty() == false)
	{
		Directory = CreateDirectory(DirectoryName, Directory);
		Path.RightChopInline(DirectoryName.Len() + 1);
		DirectoryName = GetNextDirectoryName(Path);
	}

	return Directory;
}

uint32 FIoContainerMetaWriter::AddFile(FStringView FileName, FStringView ContainerName, uint32 Directory)
{
	check(Directory != ~uint32(0));

	const uint32			File		= FileEntries.Num();
	FIoMetaFileIndexEntry&	FileEntry	= FileEntries.AddDefaulted_GetRef();
	FileEntry.Name						= GetNameIndex(FileName);
	FileEntry.ContainerName				= GetNameIndex(ContainerName);
	FileEntry.DirectoryEntry			= Directory;

	FIoMetaDirectoryIndexEntry& DirectoryEntry	= DirectoryEntries[Directory];
	FileEntry.NextFileEntry						= DirectoryEntry.FirstFileEntry;
	DirectoryEntry.FirstFileEntry				= File;

	return File;
}

uint32 FIoContainerMetaWriter::GetNameIndex(const FStringView& String)
{
	FUtf8String StringEntry(String);
	if (const uint32* Index = StringToIndex.Find(StringEntry))
	{
		return *Index;
	}
	else
	{
		const uint32 NewIndex = Strings.AddElement(MoveTemp(StringEntry));
		StringToIndex.Add(Strings[NewIndex], NewIndex);
		return NewIndex;
	}
}

FIoContainerMetaResourceView FIoContainerMetaResourceView::MakeView(FMemoryView Memory)
{
	if (Memory.GetSize() < sizeof(FIoContainerMetaHeader))
	{
		return FIoContainerMetaResourceView();
	}
	
	FIoContainerMetaResourceView View;
	View.Header = reinterpret_cast<const FIoContainerMetaHeader*>(Memory.GetData());

	if (View.Header->IsValid() == false)
	{
		return View;
	}
	Memory.RightChopInline(sizeof(FIoContainerMetaHeader));

	View.ChunkIds = MakeConstArrayView(reinterpret_cast<const FIoChunkId*>(Memory.GetData()), View.Header->FileCount);
	Memory.RightChopInline(View.ChunkIds.GetTypeSize() * View.ChunkIds.Num());

	View.FileEntryIndices = MakeConstArrayView(reinterpret_cast<const uint32*>(Memory.GetData()), View.Header->FileCount);
	Memory.RightChopInline(View.FileEntryIndices.GetTypeSize() * View.FileEntryIndices.Num());

	View.FileEntries = MakeConstArrayView(reinterpret_cast<const FIoMetaFileIndexEntry*>(Memory.GetData()), View.Header->FileCount);
	Memory.RightChopInline(View.FileEntries.GetTypeSize() * View.FileEntries.Num());

	View.DirectoryEntries = MakeConstArrayView(reinterpret_cast<const FIoMetaDirectoryIndexEntry*>(Memory.GetData()), View.Header->DirectoryCount);
	Memory.RightChopInline(View.DirectoryEntries.GetTypeSize() * View.DirectoryEntries.Num());

	View.StringTableEntries	= MakeConstArrayView(reinterpret_cast<const FIoMetaStringTableEntry*>(Memory.GetData()), View.Header->StringCount);
	View.StringTable = Memory.RightChop(View.StringTableEntries.GetTypeSize() * View.StringTableEntries.Num());

	return View;
}
