// Copyright Epic Games, Inc. All Rights Reserved.

#include "IO/IoDirectoryIndex.h"

#include "Containers/ArrayView.h"
#include "Containers/StringView.h"
#include "CoreTypes.h"
#include "IO/IoDispatcher.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CString.h"
#include "Misc/Paths.h"
#include "Misc/PathViews.h"
#include "Serialization/Archive.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "Templates/AlignmentTemplates.h"
#include "Templates/Function.h"
#include "Templates/UnrealTemplate.h"

namespace IoDirectoryIndexUtils
{

// See PakFileUtilities.cpp 
static FString GetLongestPath(const TArray<FStringView>& Filenames)
{
	FString LongestPath;
	int32 MaxNumDirectories = 0;

	for (const FStringView& Filename : Filenames)
	{
		int32 NumDirectories = 0;
		for (int32 Index = 0; Index < Filename.Len(); Index++)
		{
			if (Filename[Index] == '/')
			{
				NumDirectories++;
			}
		}
		if (NumDirectories > MaxNumDirectories)
		{
			LongestPath = Filename;
			MaxNumDirectories = NumDirectories;
		}
	}
	return FPaths::GetPath(LongestPath) + TEXT("/");
}

// See PakFileUtilities.cpp 
FString GetCommonRootPath(const TArray<FStringView>& Filenames)
{
	FString Root = GetLongestPath(Filenames);
	for (const FStringView& Filename : Filenames)
	{
		FString Path = FPaths::GetPath(FString(Filename)) + TEXT("/");
		int32 CommonSeparatorIndex = -1;
		int32 SeparatorIndex = Path.Find(TEXT("/"), ESearchCase::CaseSensitive);
		while (SeparatorIndex >= 0)
		{
			if (FCString::Strnicmp(*Root, *Path, SeparatorIndex + 1) != 0)
			{
				break;
			}
			CommonSeparatorIndex = SeparatorIndex;
			if (CommonSeparatorIndex + 1 < Path.Len())
			{
				SeparatorIndex = Path.Find(TEXT("/"), ESearchCase::CaseSensitive, ESearchDir::FromStart, CommonSeparatorIndex + 1);
			}
			else
			{
				break;
			}
		}
		if ((CommonSeparatorIndex + 1) < Root.Len())
		{
			Root.MidInline(0, CommonSeparatorIndex + 1, EAllowShrinking::No);
		}
	}
	return Root;
}

// See IPlatformFilePak.h
static bool SplitPathInline(FStringView& InOutPath, FStringView& OutFilename)
{
	// FPaths::GetPath doesn't handle our / at the end of directories, so we have to do string manipulation ourselves
	// The manipulation is less complicated than GetPath deals with, since we have normalized/path/strings, we have relative paths only, and we don't care about extensions
	if (InOutPath.Len() == 0)
	{
		check(false); // Filenames should have non-zero length, and the minimum directory length is 1 (The root directory is written as "/")
		return false;
	}
	else if (InOutPath.Len() == 1)
	{
		if (InOutPath[0] == TEXT('/'))
		{
			// The root directory; it has no parent.
			OutFilename = FStringView();
			return false;
		}
		else
		{
			// A relative one-character path with no /; this is a direct child of in the root directory
			OutFilename = TEXT("/");
			Swap(OutFilename, InOutPath);
			return true;
		}
	}
	else
	{
		if (InOutPath[InOutPath.Len() - 1] == TEXT('/'))
		{
			// The input was a Directory; remove the trailing / since we don't keep those on the CleanFilename
			InOutPath.LeftChopInline(1);
		}

		int32 Offset = 0;
		if (InOutPath.FindLastChar(TEXT('/'), Offset))
		{
			int32 FilenameStart = Offset + 1;
			OutFilename = InOutPath.Mid(FilenameStart);
			InOutPath.LeftInline(FilenameStart); // The Parent Directory keeps the / at the end
		}
		else
		{
			// A relative path with no /; this is a direct child of in the root directory
			OutFilename = TEXT("/");
			Swap(OutFilename, InOutPath);
		}
		return true;
	}
}

} // IoDirectoryIndexUtils

FArchive& operator<<(FArchive& Ar, FIoDirectoryIndexEntry& Entry)
{
	Ar << Entry.Name;
	Ar << Entry.FirstChildEntry;
	Ar << Entry.NextSiblingEntry;
	Ar << Entry.FirstFileEntry;

	return Ar;
}

FArchive& operator<<(FArchive& Ar, FIoFileIndexEntry& Entry)
{
	Ar << Entry.Name;
	Ar << Entry.NextFileEntry;
	Ar << Entry.UserData;

	return Ar;
}

FArchive& operator<<(FArchive& Ar, FIoDirectoryIndexResource& DirectoryIndex)
{
	Ar << DirectoryIndex.MountPoint;
	Ar << DirectoryIndex.DirectoryEntries;
	Ar << DirectoryIndex.FileEntries;
	Ar << DirectoryIndex.StringTable;

	return Ar;
}

static FStringView GetNextDirectoryName(const FStringView& Path)
{
	int32 SeparatorIndex = INDEX_NONE;
	if (Path.FindChar(TEXT('/'), SeparatorIndex))
	{
		return Path.Left(SeparatorIndex);
	}

	return FStringView();
}

FIoDirectoryIndexWriter::FIoDirectoryIndexWriter()
{
	// Add root entry
	DirectoryEntries.AddDefaulted();
}

void FIoDirectoryIndexWriter::SetMountPoint(FString InMountPoint)
{
	MountPoint = MoveTemp(InMountPoint);
}

uint32 FIoDirectoryIndexWriter::AddFile(const FStringView& InFileName)
{
	uint32 Directory = 0; //Root

	FStringView RelativePathFromMount = InFileName.Mid(MountPoint.Len());
	FStringView RelativeDirectoryFromMount = RelativePathFromMount;
	FStringView CleanFileName;
	IoDirectoryIndexUtils::SplitPathInline(RelativeDirectoryFromMount, CleanFileName);

	FStringView Path = RelativeDirectoryFromMount;
	FStringView DirectoryName = GetNextDirectoryName(Path);
	while (!DirectoryName.IsEmpty())
	{
		Directory = CreateDirectory(DirectoryName, Directory);
		Path.RightChopInline(DirectoryName.Len() + 1);
		DirectoryName = GetNextDirectoryName(Path);
	}

	check(Directory != ~uint32(0));

	return AddFile(CleanFileName, Directory);
}

void FIoDirectoryIndexWriter::SetFileUserData(uint32 InFileEntryIndex, uint32 InUserData)
{
	check(InFileEntryIndex != ~uint32(0));

	FileEntries[InFileEntryIndex].UserData = InUserData;
}

void FIoDirectoryIndexWriter::Flush(TArray<uint8>& OutBuffer, FAES::FAESKey EncryptionKey)
{
	TArray<FString> StringTable;
	StringTable.Reserve(Strings.Num());
	for (FString& String : Strings)
	{
		StringTable.Emplace(MoveTemp(String));
	}
	Strings.Empty();

	//TODO: Sort entries based on directory hierarchy
	FIoDirectoryIndexResource DirectoryIndex;
	DirectoryIndex.MountPoint = MoveTemp(MountPoint);
	DirectoryIndex.DirectoryEntries = MoveTemp(DirectoryEntries);
	DirectoryIndex.FileEntries = MoveTemp(FileEntries);
	DirectoryIndex.StringTable = MoveTemp(StringTable);
	
	FMemoryWriter Ar(OutBuffer);
	Ar << DirectoryIndex;

	if (EncryptionKey.IsValid())
	{
		const uint32 DirectoryIndexSize = OutBuffer.Num();
		const uint32 Alignment = FAES::AESBlockSize;
		if (!IsAligned(DirectoryIndexSize, Alignment))
		{
			const uint32 Padding = (Alignment - (DirectoryIndexSize % Alignment)) % Alignment;
			if (Padding > 0)
			{
				OutBuffer.AddUninitialized(Padding);
				for (int32 FillIndex = DirectoryIndexSize; FillIndex < OutBuffer.Num(); ++FillIndex)
				{
					OutBuffer[FillIndex] = OutBuffer[(FillIndex - DirectoryIndexSize) % DirectoryIndexSize];
				}
			}
		}

		FAES::EncryptData(OutBuffer.GetData(), OutBuffer.Num(), EncryptionKey);
	}
}

uint32 FIoDirectoryIndexWriter::GetDirectory(uint32 DirectoryName, uint32 Parent)
{
	uint32 Directory = DirectoryEntries[Parent].FirstChildEntry;
	while (IsValid(Directory))
	{
		const FIoDirectoryIndexEntry& Entry = DirectoryEntries[Directory];
		if (Entry.Name == DirectoryName)
		{
			return Directory;
		}
		Directory = Entry.NextSiblingEntry;
	}

	return ~uint32(0);
}

uint32 FIoDirectoryIndexWriter::CreateDirectory(const FStringView& DirectoryName, uint32 Parent)
{
	uint32 Name = GetNameIndex(DirectoryName);
	uint32 Directory = GetDirectory(Name, Parent);

	if (IsValid(Directory))
	{
		return Directory;
	}

	Directory = DirectoryEntries.Num();
	FIoDirectoryIndexEntry& NewEntry = DirectoryEntries.AddDefaulted_GetRef();
	NewEntry.Name = Name;
	NewEntry.NextSiblingEntry = DirectoryEntries[Parent].FirstChildEntry;
	DirectoryEntries[Parent].FirstChildEntry = Directory;

	return Directory;
}

uint32 FIoDirectoryIndexWriter::GetNameIndex(const FStringView& String)
{
	if (const uint32* Index = StringToIndex.Find(String))
	{
		return *Index;
	}
	else
	{
		const uint32 NewIndex = Strings.Add();
		const FString& NewString = Strings[NewIndex] = FString(String);
		StringToIndex.Add(NewString, NewIndex);
		return NewIndex;
	}
}

uint32 FIoDirectoryIndexWriter::AddFile(const FStringView& FileName, uint32 Directory)
{
	uint32 NewFileIdx = FileEntries.Num();
	FIoFileIndexEntry& FileEntry = FileEntries.AddDefaulted_GetRef();
	FileEntry.Name = GetNameIndex(FileName);

	FIoDirectoryIndexEntry& DirectoryEntry = DirectoryEntries[Directory];
	FileEntry.NextFileEntry = DirectoryEntry.FirstFileEntry;
	DirectoryEntry.FirstFileEntry = NewFileIdx;

	return NewFileIdx;
}

class FIoDirectoryIndexReaderImpl
{
public:
	FIoStatus Initialize(TConstArrayView<uint8> InBuffer, FAES::FAESKey InDecryptionKey)
	{
		if (InBuffer.Num() == 0)
		{
			return FIoStatus::Invalid;
		}

		if (InDecryptionKey.IsValid())
		{
			TArray<uint8, TInlineAllocator<2048>> Bytes(InBuffer); // temp buffer for dencryption
			FAES::DecryptData(Bytes.GetData(), Bytes.Num(), InDecryptionKey);
			FMemoryReaderView Ar(MakeArrayView<const uint8>(Bytes.GetData(), Bytes.Num()));
			Ar << DirectoryIndex;
		}
		else
		{
			FMemoryReaderView Ar(MakeArrayView<const uint8>(InBuffer.GetData(), InBuffer.Num()));
			Ar << DirectoryIndex;
		}

		return FIoStatus::Ok;
	}

	const FString& GetMountPoint() const
	{
		return DirectoryIndex.MountPoint;
	}

	FIoDirectoryIndexHandle GetChildDirectory(FIoDirectoryIndexHandle Directory) const
	{
		return Directory.IsValid() && IsValidIndex()
			? FIoDirectoryIndexHandle::FromIndex(GetDirectoryEntry(Directory).FirstChildEntry)
			: FIoDirectoryIndexHandle::Invalid();
	}

	FIoDirectoryIndexHandle GetNextDirectory(FIoDirectoryIndexHandle Directory) const
	{
		return Directory.IsValid() && IsValidIndex()
			? FIoDirectoryIndexHandle::FromIndex(GetDirectoryEntry(Directory).NextSiblingEntry)
			: FIoDirectoryIndexHandle::Invalid();
	}

	FIoDirectoryIndexHandle GetFile(FIoDirectoryIndexHandle Directory) const
	{
		return Directory.IsValid() && IsValidIndex()
			? FIoDirectoryIndexHandle::FromIndex(GetDirectoryEntry(Directory).FirstFileEntry)
			: FIoDirectoryIndexHandle::Invalid();
	}

	FIoDirectoryIndexHandle GetNextFile(FIoDirectoryIndexHandle File) const
	{
		return File.IsValid() && IsValidIndex()
			? FIoDirectoryIndexHandle::FromIndex(GetFileEntry(File).NextFileEntry)
			: FIoDirectoryIndexHandle::Invalid();
	}
	
	FStringView GetDirectoryName(FIoDirectoryIndexHandle Directory) const
	{
		if (Directory.IsValid() && IsValidIndex())
		{
			uint32 NameIndex = GetDirectoryEntry(Directory).Name;
			return DirectoryIndex.StringTable[NameIndex];
		}

		return FStringView();
	}

	FStringView GetFileName(FIoDirectoryIndexHandle File) const
	{
		if (File.IsValid() && IsValidIndex())
		{
			uint32 NameIndex = GetFileEntry(File).Name;
			return DirectoryIndex.StringTable[NameIndex];
		}

		return FStringView();
	}

	uint32 GetFileData(FIoDirectoryIndexHandle File) const
	{
		return File.IsValid() && IsValidIndex()
			? DirectoryIndex.FileEntries[File.ToIndex()].UserData
			: ~uint32(0);
	}

	bool IterateDirectoryIndex(FIoDirectoryIndexHandle DirectoryIndexHandle, FStringView Path, FDirectoryIndexVisitorFunction Visit)
	{
		FIoDirectoryIndexHandle File = GetFile(DirectoryIndexHandle);
		while (File.IsValid())
		{
			const uint32 TocEntryIndex = GetFileData(File);
			FStringView FileName = GetFileName(File);
			TStringBuilder<256> FilePathBuilder;
			FPathViews::Append(FilePathBuilder, GetMountPoint(), Path, FileName);
			const FStringView FilePath = FilePathBuilder.ToView();

			if (!Visit(FilePath, TocEntryIndex))
			{
				return false;
			}

			File = GetNextFile(File);
		}

		FIoDirectoryIndexHandle ChildDirectory = GetChildDirectory(DirectoryIndexHandle);
		while (ChildDirectory.IsValid())
		{
			FStringView DirectoryName = GetDirectoryName(ChildDirectory);
			TStringBuilder<256> ChildDirectoryPathBuilder;
			FPathViews::Append(ChildDirectoryPathBuilder, Path, DirectoryName);
			const FStringView ChildDirectoryPath = ChildDirectoryPathBuilder.ToView();

			if (!IterateDirectoryIndex(ChildDirectory, ChildDirectoryPath, Visit))
			{
				return false;
			}

			ChildDirectory = GetNextDirectory(ChildDirectory);
		}

		return true;
	}

private:
	const FIoDirectoryIndexEntry& GetDirectoryEntry(FIoDirectoryIndexHandle Directory) const
	{
		return DirectoryIndex.DirectoryEntries[Directory.ToIndex()];
	}

	const FIoFileIndexEntry& GetFileEntry(FIoDirectoryIndexHandle File) const
	{
		return DirectoryIndex.FileEntries[File.ToIndex()];
	}

	bool IsValidIndex() const
	{
		return DirectoryIndex.DirectoryEntries.Num() > 0;
	}

	FIoDirectoryIndexResource DirectoryIndex;
};

FIoDirectoryIndexReader::FIoDirectoryIndexReader()
	: Impl(new FIoDirectoryIndexReaderImpl)
{
}

FIoDirectoryIndexReader::~FIoDirectoryIndexReader()
{
	delete Impl;
}

FIoStatus FIoDirectoryIndexReader::Initialize(TConstArrayView<uint8> InBuffer, FAES::FAESKey InDecryptionKey)
{
	return Impl->Initialize(InBuffer, InDecryptionKey);
}

const FString& FIoDirectoryIndexReader::GetMountPoint() const
{
	return Impl->GetMountPoint();
}

FIoDirectoryIndexHandle FIoDirectoryIndexReader::GetChildDirectory(FIoDirectoryIndexHandle Directory) const
{
	return Impl->GetChildDirectory(Directory);
}

FIoDirectoryIndexHandle FIoDirectoryIndexReader::GetNextDirectory(FIoDirectoryIndexHandle Directory) const
{
	return Impl->GetNextDirectory(Directory);
}

FIoDirectoryIndexHandle FIoDirectoryIndexReader::GetFile(FIoDirectoryIndexHandle Directory) const
{
	return Impl->GetFile(Directory);
}

FIoDirectoryIndexHandle FIoDirectoryIndexReader::GetNextFile(FIoDirectoryIndexHandle File) const
{
	return Impl->GetNextFile(File);
}

FStringView FIoDirectoryIndexReader::GetDirectoryName(FIoDirectoryIndexHandle Directory) const
{
	return Impl->GetDirectoryName(Directory);
}

FStringView FIoDirectoryIndexReader::GetFileName(FIoDirectoryIndexHandle File) const
{
	return Impl->GetFileName(File);
}

uint32 FIoDirectoryIndexReader::GetFileData(FIoDirectoryIndexHandle File) const
{
	return Impl->GetFileData(File);
}

bool FIoDirectoryIndexReader::IterateDirectoryIndex(FIoDirectoryIndexHandle Directory, FStringView Path, FDirectoryIndexVisitorFunction Visit) const
{
	return Impl->IterateDirectoryIndex(Directory, Path, Visit);
}