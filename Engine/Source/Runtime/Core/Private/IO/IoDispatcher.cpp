// Copyright Epic Games, Inc. All Rights Reserved.

#include "IO/IoDispatcher.h"
#include "Async/Mutex.h"
#include "Async/UniqueLock.h"
#include "Async/Fundamental/Scheduler.h"
#include "Containers/Ticker.h"
#include "Misc/CommandLine.h"
#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "HAL/Event.h"
#include "IO/IoChunkId.h"
#include "IO/IoDispatcherBackend.h"
#include "IO/IoDispatcherConfig.h"
#include "IO/IoDispatcherChunkDecoder.h"
#include "IO/IoDispatcherInternal.h"
#include "IO/IoDispatcherPrivate.h"
#include "IO/IoOffsetLength.h"
#include "IO/PlatformIoDispatcher.h"
#include "Misc/ScopeRWLock.h"
#include "Misc/CoreDelegates.h"
#include "Misc/ScopeLock.h"
#include "ProfilingDebugging/CountersTrace.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "ProfilingDebugging/IoStoreTrace.h"
#include "ProfilingDebugging/TraceAuxiliary.h"
#include "Templates/Greater.h"

#define UE_WITH_CONTAINER_META !UE_BUILD_SHIPPING

#if UE_WITH_CONTAINER_META
#include "HAL/PlatformFileManager.h"
#include "IO/IoContainerMeta.h"
#include "Misc/Paths.h"
#else
#include "Containers/Utf8String.h"
#endif // UE_WITH_CONTAINER_META

DEFINE_LOG_CATEGORY(LogIoDispatcher);

const FIoChunkId FIoChunkId::InvalidChunkId = FIoChunkId::CreateEmptyId();

TUniquePtr<FIoDispatcher> GIoDispatcher;

#if UE_BUILD_DEVELOPMENT || UE_BUILD_DEBUG
//PRAGMA_DISABLE_OPTIMIZATION
#endif

CSV_DEFINE_CATEGORY(IoDispatcher, true);
CSV_DEFINE_STAT(IoDispatcher, PendingIoRequests);

static const TCHAR* const GetIoErrorText_ErrorCodeTextArray[] =
{
	TEXT("OK"),
	TEXT("Unknown Status"),
	TEXT("Invalid Code"),
	TEXT("Cancelled"),
	TEXT("FileOpen Failed"),
	TEXT("File Not Open"),
	TEXT("Read Error"),
	TEXT("Write Error"),
	TEXT("Not Found"),
	TEXT("Corrupt Toc"),
	TEXT("Unknown ChunkID"),
	TEXT("Invalid Parameter"),
	TEXT("Signature Error"),
	TEXT("Invalid Encryption Key"),
	TEXT("Compression Error"),
	TEXT("Pending Fork"),
	TEXT("Pending Encryption Key"),
	TEXT("Disabled"),
	TEXT("Not Installed"),
	TEXT("Pending Host Group"),
	TEXT("Timeout"),
	TEXT("Delete Error"),
	TEXT("Out Of Diskspace"),
	TEXT("FileSeek Failed"),
	TEXT("FileFlush Failed"),
	TEXT("FileMove Failed"),
	TEXT("FileClose Failed"),
	TEXT("File Corrupt"),
	TEXT("Http Read Error"),
	TEXT("File Handle Open"),
	TEXT("Pending Delete")
};
static_assert(std::size(GetIoErrorText_ErrorCodeTextArray) == static_cast<uint32>(EIoErrorCode::Last), "GetIoErrorText_ErrorCodeTextArray is missing entries to match EIoErrorCode. Please update the array.");

const TCHAR* const* GetIoErrorText_ErrorCodeText = GetIoErrorText_ErrorCodeTextArray;

FStringBuilderBase& operator<<(FStringBuilderBase& Sb, EIoContainerFlags Flags)
{
	static const TCHAR* Names[] =
	{
		TEXT("None"),
		TEXT("Compressed"),
		TEXT("Encrypted"),
		TEXT("Signed"),
		TEXT("Indexed"),
		TEXT("OnDemand")
	};
	constexpr uint32 BitCount = 1 + FMath::CountTrailingZeros(
		static_cast<std::underlying_type_t<EIoContainerFlags>>(EIoContainerFlags::Last));
	static_assert(UE_ARRAY_COUNT(Names) == BitCount + 1, "Please update names list");

	for (int32 Idx = 0; Idx < BitCount; ++Idx)
	{
		const EIoContainerFlags FlagToTest = static_cast<EIoContainerFlags>(1 << Idx);
		if (EnumHasAnyFlags(Flags, FlagToTest))
		{
			if (Sb.Len())
			{
				Sb << TEXT("|");
			}
			Sb << Names[Idx + 1];
		}
	}

	return Sb;
}

FString LexToString(EIoContainerFlags Flags)
{
	TStringBuilder<128> Sb;
	return (Sb << Flags).ToString();
}

#if UE_IODISPATCHER_STATS_ENABLED
class FIoRequestStats
{
public:
	FIoRequestStats()
#if COUNTERSTRACE_ENABLED
		: PendingIoRequestsCounter(TEXT("IoDispatcher/PendingIoRequests"), TraceCounterDisplayHint_None)
#endif
	{
		Categories.Reserve(4);
		int32 PackageDataIndex = Categories.Emplace(TEXT("PackageData"));
		int32 BulkDataIndex = Categories.Emplace(TEXT("BulkData"));
		int32 ShadersIndex = Categories.Emplace(TEXT("Shaders"));
		int32 MiscIndex = Categories.Emplace(TEXT("Misc"));

		for (int32 Index = 0; Index < UE_ARRAY_COUNT(ChunkTypeToCategoryMap); ++Index)
		{
			ChunkTypeToCategoryMap[Index] = &Categories[MiscIndex];
		}
		ChunkTypeToCategoryMap[static_cast<int32>(EIoChunkType::ExportBundleData)] = &Categories[PackageDataIndex];
		ChunkTypeToCategoryMap[static_cast<int32>(EIoChunkType::BulkData)] = &Categories[BulkDataIndex];
		ChunkTypeToCategoryMap[static_cast<int32>(EIoChunkType::OptionalBulkData)] = &Categories[BulkDataIndex];
		ChunkTypeToCategoryMap[static_cast<int32>(EIoChunkType::MemoryMappedBulkData)] = &Categories[BulkDataIndex];
		ChunkTypeToCategoryMap[static_cast<int32>(EIoChunkType::ShaderCode)] = &Categories[ShadersIndex];

#if CSV_PROFILER_STATS
		TickerHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &FIoRequestStats::TickCsv));
#endif
	}

	~FIoRequestStats()
	{
#if CSV_PROFILER_STATS
		FTSTicker::RemoveTicker(TickerHandle);
#endif
	}

	void OnBatchIssued(FIoBatch& Batch)
	{
		const uint64 StartTime = FPlatformTime::Cycles64();
		FIoRequestImpl* Request = Batch.HeadRequest;
		while (Request)
		{
			Request->StartTime = StartTime;
			Request = Request->NextRequest;
		}
	}

	void OnRequestStarted(FIoRequestImpl& Request)
	{
		++PendingIoRequests;
#if COUNTERSTRACE_ENABLED
		PendingIoRequestsCounter.Set(PendingIoRequests);
#endif
	}

	void OnRequestCompleted(FIoRequestImpl& Request)
	{
		--PendingIoRequests;
#if COUNTERSTRACE_ENABLED
		PendingIoRequestsCounter.Set(PendingIoRequests);
#endif
		if (!Request.HasBuffer())
		{
			return;
		}
		FRequestCategory* Category = ChunkTypeToCategoryMap[static_cast<int32>(Request.ChunkId.GetChunkType())];
		++Category->TotaRequestsCount;
		const double Duration = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - Request.GetStartTime());
		Category->TotalRequestsTime += Duration;
#if COUNTERSTRACE_ENABLED
		Category->TotalLoadedCounter.Add(Request.GetBuffer().DataSize());
		Category->AverageDurationCounter.Set(Category->TotalRequestsTime / double(Category->TotaRequestsCount));
#endif
	}

private:
	struct FRequestCategory
	{
		FRequestCategory(const TCHAR* Name)
#if COUNTERSTRACE_ENABLED
			: TotalLoadedCounter(TraceCounterNameType_Dynamic, *FString::Printf(TEXT("IoDispatcher/TotalLoaded (%s)"), Name), TraceCounterDisplayHint_Memory)
			, AverageDurationCounter(TraceCounterNameType_Dynamic, *FString::Printf(TEXT("IoDispatcher/AverageDuration (%s)"), Name), TraceCounterDisplayHint_None)
#endif
		{

		}

#if COUNTERSTRACE_ENABLED
		FCountersTrace::FCounterInt TotalLoadedCounter;
		FCountersTrace::FCounterFloat AverageDurationCounter;
#endif
		uint64 TotaRequestsCount = 0;
		double TotalRequestsTime = 0.0;
	};

#if CSV_PROFILER_STATS
	bool TickCsv(float DeltaTime)
	{
		CSV_CUSTOM_STAT_DEFINED(PendingIoRequests, static_cast<int32>(PendingIoRequests), ECsvCustomStatOp::Set);
		return true; // Keep ticking
	}
#endif

	TSAN_ATOMIC(int64) PendingIoRequests = 0;
#if COUNTERSTRACE_ENABLED
	FCountersTrace::FCounterInt PendingIoRequestsCounter;
#endif
	TArray<FRequestCategory> Categories;
	FRequestCategory* ChunkTypeToCategoryMap[static_cast<int32>(EIoChunkType::MAX)];
#if CSV_PROFILER_STATS
	FTSTicker::FDelegateHandle TickerHandle;
#endif
};
#else
class FIoRequestStats
{
public:
	void OnBatchIssued(FIoBatch& Batch) {}
	void OnRequestStarted(FIoRequestImpl& Request) {}
	void OnRequestCompleted(FIoRequestImpl& Request) {}
};
#endif

template <typename T, uint32 BlockSize = 128>
class TBlockAllocator
{
public:
	~TBlockAllocator()
	{
		FreeBlocks();
	}

	FORCEINLINE T* Alloc()
	{
		FScopeLock _(&CriticalSection);

		if (!NextFree)
		{
			LLM_SCOPE_BYNAME(TEXT("FileSystem/IODispatcher"));
			//TODO: Virtual alloc
			FBlock* Block = new FBlock;

			for (int32 ElementIndex = 0; ElementIndex < BlockSize; ++ElementIndex)
			{
				FElement* Element = &Block->Elements[ElementIndex];
				Element->Next = NextFree;
				NextFree = Element;
			}

			Block->Next = Blocks;
			Blocks = Block;
		}

		FElement* Element = NextFree;
		NextFree = Element->Next;

		++NumElements;

		return Element->Buffer.GetTypedPtr();
	}

	FORCEINLINE void Free(T* Ptr)
	{
		FScopeLock _(&CriticalSection);

		FElement* Element = reinterpret_cast<FElement*>(Ptr);
		Element->Next = NextFree;
		NextFree = Element;

		--NumElements;
	}

	template <typename... ArgsType>
	T* Construct(ArgsType&&... Args)
	{
		return new(Alloc()) T(Forward<ArgsType>(Args)...);
	}

	void Destroy(T* Ptr)
	{
		Ptr->~T();
		Free(Ptr);
	}

	void Trim()
	{
		FScopeLock _(&CriticalSection);
		if (!NumElements)
		{
			FreeBlocks();
		}
	}

private:
	void FreeBlocks()
	{
		FBlock* Block = Blocks;
		while (Block)
		{
			FBlock* Tmp = Block;
			Block = Block->Next;
			delete Tmp;
		}

		Blocks = nullptr;
		NextFree = nullptr;
		NumElements = 0;
	}

	struct FElement
	{
		TTypeCompatibleBytes<T> Buffer;
		FElement* Next;
	};

	struct FBlock
	{
		FElement Elements[BlockSize];
		FBlock* Next = nullptr;
	};

	FBlock*				Blocks = nullptr;
	FElement*			NextFree = nullptr;
	int32				NumElements = 0;
	FCriticalSection	CriticalSection;
};

class FIoRequestAllocator
{
public:
	void AddRef()
	{
		RefCount.IncrementExchange();
	}

	void ReleaseRef()
	{
		if (RefCount.DecrementExchange() == 1)
		{
			delete this;
		}
	}

	FIoRequestImpl* AllocRequest(const FIoChunkId& ChunkId, FIoReadOptions Options)
	{
		FIoRequestImpl* Request = BlockAllocator.Construct(*this);
		AddRef();

		Request->ChunkId = ChunkId;
		Request->Options = Options;

		return Request;
	}

	void FreeRequest(FIoRequestImpl* Request)
	{
		BlockAllocator.Destroy(Request);
		ReleaseRef();
	}

	void Trim()
	{
		BlockAllocator.Trim();
	}

private:
	TAtomic<int32> RefCount;
	TBlockAllocator<FIoRequestImpl, 4096> BlockAllocator;
};

#if UE_WITH_CONTAINER_META
class FIoMountedContainerMeta
{
	struct FContainerMeta
	{
		FContainerMeta(FIoContainerMetaReader&& InReader, FString&& InFilePath)
			: Reader(MoveTemp(InReader))
			, FilePath(MoveTemp(InFilePath))
		{ }

		FIoContainerMetaReader	Reader;
		FString					FilePath;
	};

public:
	void			Mount(const FString& FilePath);
	void			Unmount(const FString& FilePath);
	FUtf8StringView	GetFilename(const FIoChunkId& ChunkId, FUtf8StringBuilderBase& OutFilename, FUtf8StringView& OutContainerName) const;

private:
	mutable UE::FMutex					Mutex;
	TArray<TUniquePtr<FContainerMeta>>	Meta;
};

void FIoMountedContainerMeta::Mount(const FString& FilePath)
{
	LLM_SCOPE_BYNAME(TEXT("FileSystem/Debug"));

	IPlatformFile& Ipf = FPlatformFileManager::Get().GetPlatformFile();

	FString MetaPath = FPaths::ChangeExtension(FilePath, FIoContainerMetaHeader::FileExtension);
	if (Ipf.FileExists(*MetaPath) == false)
	{
		return;
	}

	TIoStatusOr<FIoContainerMetaReader> MaybeReader = FIoContainerMetaReader::Load(*MetaPath);
	if (MaybeReader.IsOk() == false)
	{
		UE_LOGF(LogIoDispatcher, Warning, "Failed to load container metadata '%ls', reason: %ls", *MetaPath, *MaybeReader.Status().ToString());
		return;
	}

	UE_LOGF(LogIoDispatcher, Log, "Mounted container metadata '%ls'", *MetaPath);
	{
		UE::TUniqueLock Lock(Mutex);
		Meta.Add(MakeUnique<FContainerMeta>(MaybeReader.ConsumeValueOrDie(), MoveTemp(MetaPath)));
	}
}

FUtf8StringView FIoMountedContainerMeta::GetFilename(const FIoChunkId& ChunkId, FUtf8StringBuilderBase& OutFilename, FUtf8StringView& OutContainerName) const
{
	FUtf8StringView Result;
	UE::TUniqueLock Lock(Mutex);
	for (const TUniquePtr<FContainerMeta>& ContainerMeta : Meta)
	{
		if (Result = ContainerMeta->Reader.GetFilename(ChunkId, OutFilename, OutContainerName); !Result.IsEmpty())
		{
			return Result;
		}
	}

	return Result;
}
#else
class FIoMountedContainerMeta
{
public:
	void			Mount(const FString& FilePath) { }
	void			Unmount(const FString& FilePath) { }
	FUtf8StringView	GetFilename(const FIoChunkId& ChunkId, FUtf8StringBuilderBase& OutFilename, FUtf8StringView& OutContainerName) const { return FUtf8StringView(); }
};
#endif // UE_WITH_CONTAINER_META

class FIoDispatcherImpl
	: public FRunnable
{
public:
	FIoDispatcherImpl(bool bInIsMultithreaded)
		: BackendContext(MakeShared<FIoDispatcherBackendContext>())
	{
		RequestAllocator = new FIoRequestAllocator();
		RequestAllocator->AddRef();
		BackendContext->WakeUpDispatcherThreadDelegate.BindRaw(this, &FIoDispatcherImpl::WakeUpDispatcherThread);
		BackendContext->bIsMultiThreaded = bInIsMultithreaded;
		MemoryTrimDelegateHandle = FCoreDelegates::GetMemoryTrimDelegate().AddLambda([this]()
		{
			RequestAllocator->Trim();
			BatchAllocator.Trim();
		});

		ChunkBlockDecoder.Initialize(GIoDispatcherDecompressionWorkerCount, GIoDispatcherMaxConsecutiveDecompressionJobs, UE::Tasks::ETaskPriority::BackgroundNormal);
		FIoChunkBlockDecoder::Set(ChunkBlockDecoder);

		OversubscriptionLimitReached = 
			LowLevelTasks::FScheduler::Get().GetOversubscriptionLimitReachedEvent().AddLambda(
				[this]()
				{
					bTaskSchedulerOversubscribed.store(true, std::memory_order_relaxed);
					DispatcherEvent->Trigger();
				}
			);

		if (IsPlatformIoDispatcherEnabled())
		{
			UE::FPlatformIoDispatcherCreateParams CreateParams
			{
				.bMultithreaded = bInIsMultithreaded,
				.bForceGeneric	= !bInIsMultithreaded
			};
#if !UE_BUILD_SHIPPING
			if (FParse::Param(FCommandLine::Get(), TEXT("forcegenericio")))
			{
				CreateParams.bForceGeneric = true;
			}
#endif
			UE::FPlatformIoDispatcher::Create(MoveTemp(CreateParams));
		}
	}

	~FIoDispatcherImpl()
	{
		delete Thread;
		for (const FBackendAndPriority& Backend : Backends)
		{
			Backend.Value->Shutdown();
		}
		FCoreDelegates::GetMemoryTrimDelegate().Remove(MemoryTrimDelegateHandle);
		BackendContext->WakeUpDispatcherThreadDelegate.Unbind();
		RequestAllocator->ReleaseRef();
		FIoChunkBlockDecoder::Release();
		UE::FPlatformIoDispatcher::Shutdown();	
	}

	void Initialize()
	{
		if (bIsInitialized)
		{
			return;
		}
		bIsInitialized = true;
		UE::FPlatformIoDispatcher::Initialize();
		if (!Backends.IsEmpty())
		{
			for (const FBackendAndPriority& Backend : Backends)
			{
				Backend.Value->Initialize(BackendContext);
				TRACE_IOSTORE_BACKEND_NAME(&Backend.Value.Get(), Backend.Value->GetName());
			}
			// If there are no mounted backends the resolve thread is not needed
			StartThread();
		}

		#if UE_TRACE_IOSTORE_ENABLED
		FTraceAuxiliary::OnConnection.AddLambda([this]()
		{
			// send the backend names again now Insights has connected
			for (const FBackendAndPriority& Backend : Backends)
			{
				TRACE_IOSTORE_BACKEND_NAME(&Backend.Value.Get(), Backend.Value->GetName());
			}
		});
		#endif //UE_TRACE_IOSTORE_ENABLED
	}

	FIoBatchImpl* AllocBatch()
	{
		LLM_SCOPE_BYNAME(TEXT("FileSystem/IODispatcher"));
		FIoBatchImpl* Batch = BatchAllocator.Construct();

		return Batch;
	}

	void WakeUpDispatcherThread()
	{
		if (BackendContext->bIsMultiThreaded)
		{
			DispatcherEvent->Trigger();
		}
	}

	void Cancel(FIoRequestImpl* Request)
	{
		if (!BackendContext->bIsMultiThreaded)
		{
			return;
		}
		Request->AddRef();
		{
			UE::TUniqueLock Lock(UpdateMutex);
			RequestsToCancel.Add(Request);
		}
		DispatcherEvent->Trigger();
	}

	void Reprioritize(FIoRequestImpl* Request)
	{
		if (!BackendContext->bIsMultiThreaded)
		{
			return;
		}
		Request->AddRef();
		{
			UE::TUniqueLock Lock(UpdateMutex);
			RequestsToReprioritize.Add(Request);
		}
		DispatcherEvent->Trigger();
	}

	TIoStatusOr<FIoMappedRegion> OpenMapped(const FIoChunkId& ChunkId, const FIoReadOptions& Options)
	{
		TIoStatusOr<FIoMappedRegion> OutResult{FIoStatus(EIoErrorCode::NotFound)};

		if (ChunkId.IsValid())
		{
			FReadScopeLock _(BackendsLock);

			for (const FBackendAndPriority& Backend : Backends)
			{
				TIoStatusOr<FIoMappedRegion> Result = Backend.Value->OpenMapped(ChunkId, Options);
				if (Result.IsOk())
				{
					OutResult = MoveTemp(Result);
					break;
				}

				if (Result.Status().GetErrorCode() != EIoErrorCode::NotFound)
				{
					if (OutResult.Status().GetErrorCode() == EIoErrorCode::NotFound ||
						OutResult.Status().GetErrorCode() == EIoErrorCode::UnknownChunkID)
					{
						OutResult = MoveTemp(Result);
					}
				}
			}
		}
		else
		{
			OutResult = FIoStatus(EIoErrorCode::InvalidParameter, TEXT("FIoChunkId is not valid"));
		}

		return OutResult;
	}

	void Mount(TSharedRef<IIoDispatcherBackend> Backend, int32 Priority)
	{
		check(IsInGameThread());

		if (bIsInitialized)
		{
			Backend->Initialize(BackendContext);
		}
		{
			FWriteScopeLock _(BackendsLock);
			int32 Index = Algo::LowerBoundBy(Backends, Priority, &FBackendAndPriority::Key, TGreater<>());
			Backends.Insert(MakeTuple(Priority, Backend), Index);
		}
		if (bIsInitialized && !Thread)
		{
			StartThread();
		}
	}

	UE_AUTORTFM_ALWAYS_OPEN bool DoesChunkExist(const FIoChunkId& ChunkId) const
	{
		FReadScopeLock _(BackendsLock);
		for (const FBackendAndPriority& Backend : Backends)
		{
			if (Backend.Value->DoesChunkExist(ChunkId))
			{
				return true;
			}
		}
		return false;
	}

	UE_AUTORTFM_ALWAYS_OPEN bool DoesChunkExist(const FIoChunkId& ChunkId, const FIoOffsetAndLength& ChunkRange) const
	{
		FReadScopeLock _(BackendsLock);
		for (const FBackendAndPriority& Backend : Backends)
		{
			if (Backend.Value->DoesChunkExist(ChunkId, ChunkRange))
			{
				return true;
			}
		}
		return false;
	}

	TIoStatusOr<uint64> GetSizeForChunk(const FIoChunkId& ChunkId) const
	{
		// Only attempt to find the size if the FIoChunkId is valid
		if (ChunkId.IsValid())
		{
			FReadScopeLock _(BackendsLock);
			for (const FBackendAndPriority& Backend : Backends)
			{
				TIoStatusOr<uint64> Result = Backend.Value->GetSizeForChunk(ChunkId);
				if (Result.IsOk())
				{
					return Result;
				}
			}
			return FIoStatus(EIoErrorCode::NotFound);
		}
		else
		{
			return FIoStatus(EIoErrorCode::InvalidParameter, TEXT("FIoChunkId is not valid"));
		}	
	}
	
	TIoStatusOr<uint64> GetSizeForChunk(const FIoChunkId& ChunkId, const FIoOffsetAndLength& ChunkRange, uint64& OutAvailable) const
	{
		// Only attempt to find the size if the FIoChunkId is valid
		if (ChunkId.IsValid())
		{
			FReadScopeLock _(BackendsLock);
			for (const FBackendAndPriority& Backend : Backends)
			{
				TIoStatusOr<uint64> Result = Backend.Value->GetSizeForChunk(ChunkId, ChunkRange, OutAvailable);
				if (Result.IsOk())
				{
					return Result;
				}
			}
			return FIoStatus(EIoErrorCode::NotFound);
		}
		else
		{
			return FIoStatus(EIoErrorCode::InvalidParameter, TEXT("FIoChunkId is not valid"));
		}	
	}

	void MountContainerMeta(const FString& FilePath)
	{
		ContainerMeta.Mount(FilePath);
	}

	FUtf8StringView	GetFilename(const FIoChunkId& ChunkId, FUtf8StringBuilderBase& OutFilename, FUtf8StringView& OutContainerName) const
	{
		return ContainerMeta.GetFilename(ChunkId, OutFilename, OutContainerName);
	}

	FIoSignatureErrorDelegate& OnSignatureError()
	{
		return BackendContext->SignatureErrorDelegate;
	}

	void IssueBatchInternal(FIoBatch& Batch, FIoBatchImpl* BatchImpl)
	{
		if (!Batch.HeadRequest)
		{
			if (BatchImpl)
			{
				CompleteBatch(BatchImpl);
			}
			return;
		}
		check(Batch.TailRequest);

		if (!HasMountedBackend())
		{
			FIoRequestImpl* Request = Batch.HeadRequest;
			while (Request)
			{
				FIoRequestImpl* NextRequest = Request->NextRequest;
				CompleteRequest(Request, EIoErrorCode::NotFound);
				Request->ReleaseRef();
				Request = NextRequest;
			}
			Batch.HeadRequest = Batch.TailRequest = nullptr;
			if (BatchImpl)
			{
				CompleteBatch(BatchImpl);
			}
			return;
		}

		uint32 RequestCount = 0;
		FIoRequestImpl* Request = Batch.HeadRequest;
		while (Request)
		{
			Request->Batch = BatchImpl;
			Request = Request->NextRequest;
			++RequestCount;
		}
		if (BatchImpl)
		{
			BatchImpl->UnfinishedRequestsCount += RequestCount;
		}

		RequestStats.OnBatchIssued(Batch);
		
		{
			UE::TUniqueLock Lock(WaitingMutex);
			WaitingRequests.AddTail(Batch.HeadRequest, Batch.TailRequest);
		}
		Batch.HeadRequest = Batch.TailRequest = nullptr;
		if (BackendContext->bIsMultiThreaded)
		{
			DispatcherEvent->Trigger();
		}
		else
		{
			LLM_SCOPE_BYNAME(TEXT("FileSystem/IODispatcher"));
			ProcessIncomingRequests();
			if (UE::IPlatformIoDispatcher* PlatformDispatcher = UE::FPlatformIoDispatcher::TryGet())
			{
				for (;;)
				{
					if (PlatformDispatcher->Tick() == false)
					{
						break;
					}
				}
			}
			while (PendingIoRequestsCount > 0)
			{
				ProcessCompletedRequests();
			}
		}
	}

	void IssueBatch(FIoBatch& Batch)
	{
		IssueBatchInternal(Batch, nullptr);
	}

	void IssueBatchWithCallback(FIoBatch& Batch, TFunction<void()>&& Callback)
	{
		FIoBatchImpl* Impl = AllocBatch();
		Impl->Callback = MoveTemp(Callback);
		IssueBatchInternal(Batch, Impl);
	}

	void IssueBatchAndTriggerEvent(FIoBatch& Batch, FEvent* Event)
	{
		FIoBatchImpl* Impl = AllocBatch();
		Impl->Event = Event;
		IssueBatchInternal(Batch, Impl);
	}

	void IssueBatchAndDispatchSubsequents(FIoBatch& Batch, FGraphEventRef GraphEvent)
	{
		FIoBatchImpl* Impl = AllocBatch();
		Impl->GraphEvent = MoveTemp(GraphEvent);
		IssueBatchInternal(Batch, Impl);
	}

	int64 GetTotalLoaded() const
	{
		return TotalLoaded.load(std::memory_order_relaxed);
	}

	bool HasMountedBackend() const
	{
		FReadScopeLock _(BackendsLock);
		return Backends.Num() > 0;
	}

private:
	friend class FIoBatch;
	friend class FIoRequest;

	void StartThread()
	{
		check(!Thread);
		Thread = FRunnableThread::Create(this, TEXT("IoDispatcher"), 0, TPri_AboveNormal, FPlatformAffinity::GetIoDispatcherThreadMask());
	}

	void ProcessCompletedRequests()
	{
		//TRACE_CPUPROFILER_EVENT_SCOPE(ProcessCompletedRequests);

		FReadScopeLock _(BackendsLock);
		for (const FBackendAndPriority& Backend : Backends)
		{
			FIoRequestImpl* CompletedRequestsHead = Backend.Value->GetCompletedIoRequests();
			while (CompletedRequestsHead)
			{
				FIoRequestImpl* NextRequest = CompletedRequestsHead->NextRequest;
				if (CompletedRequestsHead->LastBackendError != EIoErrorCode::Ok)
				{
					CompleteRequest(CompletedRequestsHead, CompletedRequestsHead->LastBackendError);
				}
				else
				{
					UE_CLOGF(!CompletedRequestsHead->HasBuffer(), LogStreaming, Fatal, "Backend provided a completed request without an IoBuiffer. Requests that are not failed or cancelled must have an IoBuffer");
					TotalLoaded.fetch_add(CompletedRequestsHead->GetBuffer().DataSize(), std::memory_order_relaxed);
					CompleteRequest(CompletedRequestsHead, EIoErrorCode::Ok);
				}
				CompletedRequestsHead->ReleaseRef();
				CompletedRequestsHead = NextRequest;
				--PendingIoRequestsCount;
			}
		}
	}

	void CompleteBatch(FIoBatchImpl* Batch)
	{
		if (Batch->Callback)
		{
			Batch->Callback();
		}
		if (Batch->Event)
		{
			Batch->Event->Trigger();
		}
		if (Batch->GraphEvent)
		{
			Batch->GraphEvent->DispatchSubsequents();
		}
		BatchAllocator.Destroy(Batch);
	}

	bool CompleteRequest(FIoRequestImpl* Request, EIoErrorCode Status)
	{
		//TRACE_CPUPROFILER_EVENT_SCOPE(CompleteRequest);

		EIoErrorCode ExpectedStatus = EIoErrorCode::Unknown;
		if (!Request->ErrorCode.CompareExchange(ExpectedStatus, Status))
		{
			return false;
		}

		RequestStats.OnRequestCompleted(*Request);

		FIoBatchImpl* Batch = Request->Batch;
		if (Request->Callback)
		{
			TIoStatusOr<FIoBuffer> Result;
			if (Status == EIoErrorCode::Ok)
			{
				Result = Request->GetBuffer();
			}
			else
			{
				Result = Status;
			}
			Request->Callback(Result);
			Request->Callback = {};
		}
		if (Batch)
		{
			check(Batch->UnfinishedRequestsCount);
			if (--Batch->UnfinishedRequestsCount == 0)
			{
				CompleteBatch(Batch);
			}
		}
		return true;
	}

	void ProcessIncomingRequests()
	{
		FIoRequestList RequestsToSubmit;
		//TRACE_CPUPROFILER_EVENT_SCOPE(ProcessIncomingRequests);
		for (;;)
		{
			{
				UE::TUniqueLock Lock(WaitingMutex);
				RequestsToSubmit.AddTail(MoveTemp(WaitingRequests));
			}
			TArray<FIoRequestImpl*> LocalRequestsToCancel;
			TArray<FIoRequestImpl*> LocalRequestsToReprioritize;
			{
				UE::TUniqueLock Lock(UpdateMutex);
				Swap(LocalRequestsToCancel, RequestsToCancel);
				Swap(LocalRequestsToReprioritize, RequestsToReprioritize);
			}
			for (FIoRequestImpl* RequestToCancel : LocalRequestsToCancel)
			{
				if (!RequestToCancel->IsCancelled())
				{
					RequestToCancel->SetLastBackendError(EIoErrorCode::Cancelled);
					if (RequestToCancel->Backend)
					{
						RequestToCancel->Backend->CancelIoRequest(RequestToCancel);
					}
				}
				RequestToCancel->ReleaseRef();
			}
			for (FIoRequestImpl* RequestToRePrioritize : LocalRequestsToReprioritize)
			{
				if (RequestToRePrioritize->Backend)
				{
					RequestToRePrioritize->Backend->UpdatePriorityForIoRequest(RequestToRePrioritize);
				}
				RequestToRePrioritize->ReleaseRef();
			}
			if (RequestsToSubmit.IsEmpty())
			{
				return;
			}

			int32 BatchCount = 0;
			FIoRequestList Batch;
			while (FIoRequestImpl* Request = RequestsToSubmit.PopHead())
			{
				check(Request->NextRequest == nullptr);
				RequestStats.OnRequestStarted(*Request);

				if (Request->IsCancelled())
				{
					CompleteRequest(Request, EIoErrorCode::Cancelled);
					Request->ReleaseRef();
					continue;
				}

				if (!Request->ChunkId.IsValid())
				{
					CompleteRequest(Request, EIoErrorCode::NotFound);
					Request->ReleaseRef();
					continue;
				}

				Batch.AddTail(Request);
				++BatchCount;
			}

			if (BatchCount > 0)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(ResolveRequest);
				FReadScopeLock _(BackendsLock);
				
				FIoRequestList Unresolved;
				for (const FBackendAndPriority& Backend : Backends)
				{
					for (FIoRequestImpl& Request : Batch)
					{
						Request.Backend = &Backend.Value.Get();
					}
					Backend.Value->ResolveIoRequests(MoveTemp(Batch), Unresolved);
					Batch = MoveTemp(Unresolved);
					if (Batch.IsEmpty())
					{
						break;
					}
				}

				Unresolved = MoveTemp(Batch);

				int32 UnresolvedCount = 0;
				while (FIoRequestImpl* Request = Unresolved.PopHead())
				{
					check(Request->NextRequest == nullptr);
					Request->Backend = nullptr;
					TRACE_IOSTORE_REQUEST_UNRESOLVED(Request);
					CompleteRequest(Request, EIoErrorCode::NotFound);
					Request->ReleaseRef();
					UnresolvedCount++;
				}

				check(UnresolvedCount <= BatchCount);
				PendingIoRequestsCount += (BatchCount - UnresolvedCount);
			}
		}
	}

	virtual bool Init()
	{
		return true;
	}

	virtual uint32 Run()
	{
		FMemory::SetupTLSCachesOnCurrentThread();

		LLM_SCOPE_BYNAME(TEXT("FileSystem/IODispatcher"));
		while (!bStopRequested)
		{
			if (PendingIoRequestsCount)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(IoDispatcherWaitForIo);
				DispatcherEvent->Wait();
			}
			else
			{
				DispatcherEvent->Wait();
			}
			ProcessIncomingRequests();
			ProcessCompletedRequests();
			while(bTaskSchedulerOversubscribed.load(std::memory_order_relaxed))
			{
				bTaskSchedulerOversubscribed = ChunkBlockDecoder.TryRetractAndExecuteDecodeTasks();
			}
		}

		ProcessIncomingRequests();
		while (PendingIoRequestsCount > 0)
		{
			ProcessCompletedRequests();
		}
		return 0;
	}

	virtual void Stop()
	{
		bStopRequested = true;
		DispatcherEvent->Trigger();
	}

	using FBatchAllocator = TBlockAllocator<FIoBatchImpl, 4096>;
	using FBackendAndPriority = TTuple<int32, TSharedRef<IIoDispatcherBackend>>;

	FIoMountedContainerMeta ContainerMeta;
	TSharedRef<FIoDispatcherBackendContext> BackendContext;
	FDelegateHandle MemoryTrimDelegateHandle;
	mutable FRWLock BackendsLock;
	TArray<FBackendAndPriority> Backends;
	FIoRequestAllocator* RequestAllocator = nullptr;
	FBatchAllocator BatchAllocator;
	FRunnableThread* Thread = nullptr;
	FEventRef DispatcherEvent;
	FIoRequestList WaitingRequests;
	TArray<FIoRequestImpl*> RequestsToCancel;
	TArray<FIoRequestImpl*> RequestsToReprioritize;
	TAtomic<bool> bStopRequested { false };
	uint64 PendingIoRequestsCount = 0;
	std::atomic<int64> TotalLoaded = 0;
	FIoRequestStats RequestStats;
	UE::FIoDispatcherChunkBlockDecoder ChunkBlockDecoder;
	bool bIsInitialized = false;
	UE::FMutex WaitingMutex;
	UE::FMutex UpdateMutex;
	FDelegateHandle OversubscriptionLimitReached;
	std::atomic_bool bTaskSchedulerOversubscribed{false};
};

FIoDispatcher::FIoDispatcher()
	: Impl(new FIoDispatcherImpl(FGenericPlatformProcess::SupportsMultithreading()))
{
}

FIoDispatcher::~FIoDispatcher()
{
	delete Impl;
}

void
FIoDispatcher::Mount(TSharedRef<IIoDispatcherBackend> Backend, int32 Priority)
{
	Impl->Mount(Backend, Priority);
}

FIoBatch
FIoDispatcher::NewBatch()
{
	return FIoBatch(*Impl);
}

TIoStatusOr<FIoMappedRegion>
FIoDispatcher::OpenMapped(const FIoChunkId& ChunkId, const FIoReadOptions& Options)
{
	return Impl->OpenMapped(ChunkId, Options);
}

// Polling methods
bool
FIoDispatcher::DoesChunkExist(const FIoChunkId& ChunkId) const
{
	return Impl->DoesChunkExist(ChunkId);
}

bool
FIoDispatcher::DoesChunkExist(const FIoChunkId& ChunkId, const FIoOffsetAndLength& ChunkRange) const
{
	return Impl->DoesChunkExist(ChunkId, ChunkRange);
}

TIoStatusOr<uint64>
FIoDispatcher::GetSizeForChunk(const FIoChunkId& ChunkId) const
{
	return Impl->GetSizeForChunk(ChunkId);
}

TIoStatusOr<uint64>
FIoDispatcher::GetSizeForChunk(const FIoChunkId& ChunkId, const FIoOffsetAndLength& ChunkRange, uint64& OutAvailable) const
{
	return Impl->GetSizeForChunk(ChunkId, ChunkRange, OutAvailable);
}

void FIoDispatcher::MountContainerMeta(const FString& FilePath)
{
	Impl->MountContainerMeta(FilePath);
}

FUtf8StringView	FIoDispatcher::GetFilename(const FIoChunkId& ChunkId, FUtf8StringBuilderBase& OutFilename, FUtf8StringView& OutContainerName) const
{
	return Impl->GetFilename(ChunkId, OutFilename, OutContainerName);
}

int64
FIoDispatcher::GetTotalLoaded() const
{
	return Impl->GetTotalLoaded();
}

FIoSignatureErrorDelegate&
FIoDispatcher::OnSignatureError()
{
	return Impl->OnSignatureError();
}

bool
FIoDispatcher::IsInitialized()
{
	return GIoDispatcher.IsValid();
}

FIoStatus
FIoDispatcher::Initialize()
{
	LLM_SCOPE_BYNAME(TEXT("FileSystem/IODispatcher"));
	check(!GIoDispatcher);
	GIoDispatcher = MakeUnique<FIoDispatcher>();
	return FIoStatus::Ok;
}

void
FIoDispatcher::InitializePostSettings()
{
	LLM_SCOPE_BYNAME(TEXT("FileSystem/IODispatcher"));
	check(GIoDispatcher);
	GIoDispatcher->Impl->Initialize();
}

void
FIoDispatcher::Shutdown()
{
	TUniquePtr<FIoDispatcher> LocalIoDispatcher(MoveTemp(GIoDispatcher));
}

FIoDispatcher&
FIoDispatcher::Get()
{
	check(GIoDispatcher);
	return *GIoDispatcher;
}

//////////////////////////////////////////////////////////////////////////

FIoBatch::FIoBatch(FIoDispatcherImpl& InDispatcher)
	: Dispatcher(&InDispatcher)
{
}

FIoBatch::FIoBatch()
	: Dispatcher(GIoDispatcher.IsValid() ? GIoDispatcher->Impl : nullptr)
{
}

FIoBatch::FIoBatch(FIoBatch&& Other)
{
	Dispatcher = Other.Dispatcher;
	HeadRequest = Other.HeadRequest;
	TailRequest = Other.TailRequest;
	Other.HeadRequest = nullptr;
	Other.TailRequest = nullptr;
}

FIoBatch::~FIoBatch()
{
	FIoRequestImpl* Request = HeadRequest;
	while (Request)
	{
		FIoRequestImpl* NextRequest = Request->NextRequest;
		Request->ReleaseRef();
		Request = NextRequest;
	}
}

FIoBatch&
FIoBatch::operator=(FIoBatch&& Other)
{
	if (&Other == this)
	{
		return *this;
	}
	FIoRequestImpl* Request = HeadRequest;
	while (Request)
	{
		FIoRequestImpl* NextRequest = Request->NextRequest;
		Request->ReleaseRef();
		Request = NextRequest;
	}
	Dispatcher = Other.Dispatcher;
	HeadRequest = Other.HeadRequest;
	TailRequest = Other.TailRequest;
	Other.HeadRequest = nullptr;
	Other.TailRequest = nullptr;
	return *this;
}

FIoRequestImpl*
FIoBatch::ReadInternal(const FIoChunkId& ChunkId, const FIoReadOptions& Options, int32 Priority)
{
	FIoRequestImpl* Request = Dispatcher->RequestAllocator->AllocRequest(ChunkId, Options);
	Request->Priority = Priority;
	Request->AddRef();
	if (!HeadRequest)
	{
		check(!TailRequest);
		HeadRequest = TailRequest = Request;
	}
	else
	{
		check(TailRequest);
		TailRequest->NextRequest = Request;
		TailRequest = Request;
	}
	TRACE_IOSTORE_REQUEST_CREATE(this, Request);
	return Request;
}

FIoRequest
FIoBatch::Read(const FIoChunkId& ChunkId, FIoReadOptions Options, int32 Priority)
{
	FIoRequestImpl* Request = ReadInternal(ChunkId, Options, Priority);
	return FIoRequest(Request);
}

FIoRequest
FIoBatch::ReadWithCallback(const FIoChunkId& ChunkId, const FIoReadOptions& Options, int32 Priority, FIoReadCallback&& Callback)
{
	FIoRequestImpl* Request = ReadInternal(ChunkId, Options, Priority);
	Request->Callback = MoveTemp(Callback);
	return FIoRequest(Request);
}

void
FIoBatch::Issue()
{
	Dispatcher->IssueBatch(*this);
}

void
FIoBatch::IssueWithCallback(TFunction<void()>&& Callback)
{
	Dispatcher->IssueBatchWithCallback(*this, MoveTemp(Callback));
}

void
FIoBatch::IssueAndTriggerEvent(FEvent* Event)
{
	Dispatcher->IssueBatchAndTriggerEvent(*this, Event);
}

void
FIoBatch::IssueAndDispatchSubsequents(FGraphEventRef Event)
{
	Dispatcher->IssueBatchAndDispatchSubsequents(*this, MoveTemp(Event));
}

//////////////////////////////////////////////////////////////////////////

void FIoRequestImpl::CreateBuffer(uint64 Size)
{
	if (void* TargetVa = Options.GetTargetVa())
	{
		Buffer.Emplace(FIoBuffer::Wrap, TargetVa, Size);
	}
	else
	{
		UE::FInheritedContextScope InheritedContextScope = RestoreInheritedContext();
		TRACE_CPUPROFILER_EVENT_SCOPE(AllocMemoryForRequest);
		Buffer.Emplace(Size);
	}
}

void FIoRequestImpl::FreeRequest()
{
	Allocator.FreeRequest(this);
}

FIoRequest::FIoRequest(FIoRequestImpl* InImpl)
	: Impl(InImpl)
{
	if (Impl)
	{
		Impl->AddRef();
	}
}

FIoRequest::FIoRequest(const FIoRequest& Other)
{
	if (Other.Impl)
	{
		Impl = Other.Impl;
		Impl->AddRef();
	}
}

FIoRequest::FIoRequest(FIoRequest&& Other)
{
	Impl = Other.Impl;
	Other.Impl = nullptr;
}

FIoRequest& FIoRequest::operator=(const FIoRequest& Other)
{
	if (Other.Impl)
	{
		Other.Impl->AddRef();
	}
	if (Impl)
	{
		Impl->ReleaseRef();
	}
	Impl = Other.Impl;
	return *this;
}

FIoRequest& FIoRequest::operator=(FIoRequest&& Other)
{
	if (Impl)
	{
		Impl->ReleaseRef();
	}
	Impl = Other.Impl;
	Other.Impl = nullptr;
	return *this;
}

FIoRequest::~FIoRequest()
{
	if (Impl)
	{
		Impl->ReleaseRef();
	}
}

FIoStatus
FIoRequest::Status() const
{
	if (Impl)
	{
		return Impl->ErrorCode.Load();
	}
	else
	{
		return FIoStatus::Invalid;
	}
}

const FIoBuffer*
FIoRequest::GetResult() const
{
	if (!Impl)
	{
		return nullptr;
	}
	FIoStatus Status(Impl->ErrorCode.Load());
	check(Status.IsCompleted());
	TIoStatusOr<FIoBuffer> Result;
	if (Status.IsOk())
	{
		return &Impl->GetBuffer();
	}
	else
	{
		return nullptr;
	}
}

const FIoBuffer&
FIoRequest::GetResultOrDie() const
{
	const FIoBuffer* Result = GetResult();
	if (!Result)
	{
		if (Impl)
		{
			UE_LOGF(LogIoDispatcher, Fatal, "I/O Error '%ls' (IoChunkId='%ls', Type=%ls)", *Status().ToString(), *LexToString(Impl->ChunkId), *LexToString(Impl->ChunkId.GetChunkType()));
		}
		else
		{
			UE_LOGF(LogIoDispatcher, Fatal, "I/O Error '%ls'", *Status().ToString());
		}
	}
	return *Result;
}

void
FIoRequest::Cancel()
{
	if (!GIoDispatcher || !Impl)
	{
		return;
	}
	//TRACE_BOOKMARK(TEXT("FIoRequest::Cancel()"));
	GIoDispatcher->Impl->Cancel(Impl);
}

void
FIoRequest::UpdatePriority(int32 NewPriority)
{
	if (!GIoDispatcher || !Impl || Impl->Priority == NewPriority)
	{
		return;
	}
	Impl->Priority = NewPriority;
	GIoDispatcher->Impl->Reprioritize(Impl);
}

void
FIoRequest::Release()
{
	if (Impl)
	{
		Impl->ReleaseRef();
		Impl = nullptr;
	}
}

bool
FIoDispatcherInternal::HasPackageData()
{
	// Checking for the global script objects chunk is currently the best means 
	// to know if the IoDispatcher will be loading packaged data
	const FIoDispatcher& Dispatcher = FIoDispatcher::Get();
	static const bool HasScriptObjectsChunk = Dispatcher.DoesChunkExist(CreateIoChunkId(0, 0, EIoChunkType::ScriptObjects));
	return HasScriptObjectsChunk;
}

void
FIoDispatcherInternal::MountContainerMeta(const FString& FilePath)
{
	FIoDispatcher::Get().MountContainerMeta(FilePath);
}

FUtf8StringView
FIoDispatcherInternal::GetFilename(const FIoChunkId& ChunkId, FUtf8StringBuilderBase& OutFilename, FUtf8StringView& OutContainerName)
{
	return FIoDispatcher::Get().GetFilename(ChunkId, OutFilename, OutContainerName);
}

void
FIoDispatcherInternal::GetDebugInfo(const FIoRequest& IoRequest, FIoRequestDebugInfo& OutDebugInfo)
{
	if (IoRequest.Impl && IoRequest.Impl->Backend)
	{
		OutDebugInfo.BackendName = IoRequest.Impl->Backend->GetName();
	}
}
