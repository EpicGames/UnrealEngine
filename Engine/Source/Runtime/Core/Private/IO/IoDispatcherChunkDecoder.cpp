// Copyright Epic Games, Inc. All Rights Reserved.

#include "IO/IoDispatcherChunkDecoder.h"
#include "ProfilingDebugging/CountersTrace.h"

namespace UE
{

TRACE_DECLARE_INT_COUNTER(ChunkDecoderQueueCount, TEXT("IoDispatcher/ChunkDecoderQueueCount"));
TRACE_DECLARE_INT_COUNTER(AvailableChunkDecoderWorkerCount, TEXT("IoDispatcher/AvailableChunkDecoderWorkerCount"));

void FIoDispatcherChunkBlockDecoder::Enqueue(FIoChunkBlockDecodeRequest&& DecodeRequest)
{
	check(DecodeRequest.IsValid());

	FDecodeContext* Ctx = nullptr;
	{
		UE::TUniqueLock Lock(Mutex);
		if (NextFreeContext != nullptr)
		{
			Ctx				= NextFreeContext;
			NextFreeContext	= Ctx->Next;
			Ctx->Next		= nullptr;
			TRACE_COUNTER_DECREMENT(AvailableChunkDecoderWorkerCount);
		}
		else
		{
			QueueEntries.AddTail(QueueEntryAllocator.Construct(MoveTemp(DecodeRequest)));
			TRACE_COUNTER_INCREMENT(ChunkDecoderQueueCount);
		}
	}

	if (Ctx != nullptr)
	{
		LaunchDecodeTask(*Ctx, MoveTemp(DecodeRequest));
	}
}

void FIoDispatcherChunkBlockDecoder::Initialize(uint32 InMaxWorkerCount, uint32 InMaxConsecutiveDecodeJobs, UE::Tasks::ETaskPriority InTaskPriority)
{
	MaxWorkerCount				= FMath::Clamp(InMaxWorkerCount, 1u, 16u);
	MaxConsecutiveDecodeJobs	= FMath::Clamp(InMaxConsecutiveDecodeJobs, 1u, 16u);
	TaskPriority				= InTaskPriority;

	DecodeContexts.Reserve(MaxWorkerCount);
	DecodeContexts.SetNum(MaxWorkerCount);

	for (FDecodeContext& Ctx : DecodeContexts)
	{
		Ctx.Next		= NextFreeContext;
		NextFreeContext	= &Ctx;
		TRACE_COUNTER_INCREMENT(AvailableChunkDecoderWorkerCount);
	}
}

bool FIoDispatcherChunkBlockDecoder::TryRetractAndExecuteDecodeTasks()
{
	int32 ContextIndex		= 0;
	bool bOversubscribed	= LowLevelTasks::FScheduler::Get().IsOversubscriptionLimitReached(TaskPriority);

	while (bOversubscribed)
	{
		UE::Tasks::FTask Task;
		{
			if (ContextIndex >= DecodeContexts.Num())
			{
				return false; // nothing else to do
			}
			UE::TUniqueLock Lock(Mutex);
			Task = DecodeContexts[ContextIndex].Task;
			ContextIndex++;
		}
		if (Task.IsValid())
		{
			Task.TryRetractAndExecute();
		}
		bOversubscribed = LowLevelTasks::FScheduler::Get().IsOversubscriptionLimitReached(TaskPriority);
	}

	return bOversubscribed;
}

bool FIoDispatcherChunkBlockDecoder::TryExecuteDecodeRequest()
{
	FQueueEntry* QueueEntry = nullptr;
	{
		UE::TUniqueLock Lock(Mutex);
		if (QueueEntry = QueueEntries.PopHead(); QueueEntry != nullptr)
		{
			TRACE_COUNTER_DECREMENT(ChunkDecoderQueueCount);
		}
	}

	if (QueueEntry == nullptr)
	{
		return false;
	}

	FIoChunkBlockDecodeRequest NextDecodeRequest;
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(IoDispatcherDecodeBlock);
		ProcessDecodeRequest(MoveTemp(QueueEntry->Request), NextDecodeRequest);
	}

	bool bDequeueAndLaunchDecodeTasks = false;
	{
		UE::TUniqueLock Lock(Mutex);
		if (NextDecodeRequest.IsValid())
		{
			QueueEntry->Request = MoveTemp(NextDecodeRequest);
			QueueEntries.AddTail(QueueEntry);
			TRACE_COUNTER_INCREMENT(ChunkDecoderQueueCount);
		}
		else
		{
			QueueEntryAllocator.Destroy(QueueEntry);
		}
		bDequeueAndLaunchDecodeTasks = QueueEntries.IsEmpty() == false;
	}

	if (bDequeueAndLaunchDecodeTasks)
	{
		TryDequeueAndLaunchDecodeTasks();
	}

	return true;
}

void FIoDispatcherChunkBlockDecoder::TryDequeueAndLaunchDecodeTasks()
{
	for(;;)
	{
		FIoChunkBlockDecodeRequest DecodeRequest;
		FDecodeContext* Ctx	= nullptr;
		{
			UE::TUniqueLock Lock(Mutex);
			if (NextFreeContext == nullptr || QueueEntries.PeekHead() == nullptr)
			{
				break;
			}

			Ctx						= NextFreeContext;
			NextFreeContext			= Ctx->Next;
			Ctx->Next				= nullptr;

			FQueueEntry* QueueEntry	= QueueEntries.PopHead();
			DecodeRequest			= MoveTemp(QueueEntry->Request);
			QueueEntryAllocator.Destroy(QueueEntry);
			TRACE_COUNTER_DECREMENT(ChunkDecoderQueueCount);
			TRACE_COUNTER_DECREMENT(AvailableChunkDecoderWorkerCount);
		}

		check(DecodeRequest.IsValid());
		LaunchDecodeTask(*Ctx, MoveTemp(DecodeRequest));
	}
}

void FIoDispatcherChunkBlockDecoder::LaunchDecodeTask(FDecodeContext& Ctx, FIoChunkBlockDecodeRequest&& DecodeRequest)
{
	UE::Tasks::FTask PrevTask = MoveTemp(Ctx.Task);
	Ctx.Task = UE::Tasks::Launch(
		TEXT("IoChunkDecodeBlockTask"),
		[this, &Ctx, DecodeRequest = MoveTemp(DecodeRequest)]() mutable
		{
			check(Ctx.Next == nullptr);
			check(DecodeRequest.IsValid());

			uint32 ConsecutiveJobCount = 0;
			while (DecodeRequest.IsValid() && ConsecutiveJobCount <= MaxConsecutiveDecodeJobs) 
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(IoDispatcherDecodeBlock);
				FIoChunkBlockDecodeRequest NextDecodeRequest;
				ProcessDecodeRequest(MoveTemp(DecodeRequest), NextDecodeRequest);
				DecodeRequest = MoveTemp(NextDecodeRequest);
				++ConsecutiveJobCount;
			}

			bool bDequeueAndLaunchDecodeTasks = false;
			{
				UE::TUniqueLock Lock(Mutex);
				Ctx.Next		= NextFreeContext;
				NextFreeContext = &Ctx;
				TRACE_COUNTER_INCREMENT(AvailableChunkDecoderWorkerCount);

				if (DecodeRequest.IsValid())
				{
					QueueEntries.AddTail(QueueEntryAllocator.Construct(MoveTemp(DecodeRequest)));
					TRACE_COUNTER_INCREMENT(ChunkDecoderQueueCount);
				}
				bDequeueAndLaunchDecodeTasks = QueueEntries.IsEmpty() == false;
			}

			if (bDequeueAndLaunchDecodeTasks)
			{
				TryDequeueAndLaunchDecodeTasks();
			}
		});
}

void FIoDispatcherChunkBlockDecoder::ProcessDecodeRequest(FIoChunkBlockDecodeRequest&& DecodeRequest, FIoChunkBlockDecodeRequest& OutNext)
{
	FIoChunkBlockDecodeResult DecodeResult = FIoChunkEncoding::DecodeBlock(DecodeRequest.Params, DecodeRequest.EncodedBlock, DecodeRequest.DecodedBlock);
	FIoBlockDecoded OnDecoded = MoveTemp(DecodeRequest.OnDecoded);
	check(OnDecoded.IsSet());
	OnDecoded(MoveTemp(DecodeResult), OutNext);
}

} // namespace UE
