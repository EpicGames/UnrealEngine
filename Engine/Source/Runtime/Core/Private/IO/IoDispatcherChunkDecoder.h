// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Mutex.h"
#include "IO/IoAllocators.h"
#include "IO/IoContainers.h"
#include "IO/IoChunkEncoding.h"
#include "Tasks/Task.h"

namespace UE
{

class FIoDispatcherChunkBlockDecoder
	: public IIoChunkBlockDecoder
{
	struct FDecodeContext
	{
		UE::Tasks::FTask	Task;
		FDecodeContext*		Next = nullptr;
	};

	struct FQueueEntry
		: public TIntrusiveListElement<FQueueEntry>
	{
		FQueueEntry(FIoChunkBlockDecodeRequest&& InRequest)
			: Request(MoveTemp(InRequest))
		{ }

		FIoChunkBlockDecodeRequest	Request;
		FQueueEntry*				Next = nullptr;
	};

	using FQueueEntryAllocator	= TSingleThreadedSlabAllocator<FQueueEntry, 512>;
	using FQueue				= TIntrusiveList<FQueueEntry>;

public:
								FIoDispatcherChunkBlockDecoder() = default;
	virtual						~FIoDispatcherChunkBlockDecoder() = default;

	// IIoChunkBlockDecoder
	virtual void				Enqueue(FIoChunkBlockDecodeRequest&& DecodeRequest) override;

	void						Initialize(uint32 MaxWorkerCount, uint32 MaxConsecutiveDecodeJobs, UE::Tasks::ETaskPriority TaskPriority);
	bool						TryRetractAndExecuteDecodeTasks();
	bool						TryExecuteDecodeRequest();
	void						TryDequeueAndLaunchDecodeTasks();

private:
	void						LaunchDecodeTask(FDecodeContext& Ctx, FIoChunkBlockDecodeRequest&& DecodeRequest);
	void						ProcessDecodeRequest(FIoChunkBlockDecodeRequest&& DecodeRequest, FIoChunkBlockDecodeRequest& OutNext);

	
	TArray<FDecodeContext>		DecodeContexts;	
	FDecodeContext*				NextFreeContext = nullptr;
	FQueueEntryAllocator		QueueEntryAllocator;
	FQueue						QueueEntries;
	uint32						MaxWorkerCount = 0;
	uint32						MaxConsecutiveDecodeJobs = 0;
	UE::FMutex					Mutex;
	UE::Tasks::ETaskPriority	TaskPriority = UE::Tasks::ETaskPriority::BackgroundNormal;
};

} // namespace UE
