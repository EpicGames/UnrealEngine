// Copyright Epic Games, Inc. All Rights Reserved.

#include "IO/IoHash.h"

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Memory/CompositeBuffer.h"
#include "Serialization/Archive.h"

FIoHash FIoHash::HashBuffer(const FCompositeBuffer& Buffer)
{
	return FBlake3::HashBuffer(Buffer);
}

FArchive& operator<<(FArchive& Ar, FIoHash& InHash)
{
	Ar.Serialize(InHash.GetBytes(), sizeof(decltype(InHash.GetBytes())));
	return Ar;
}

FString LexToString(const FIoHash& Hash)
{
	FString Output;
	TArray<TCHAR, FString::AllocatorType>& CharArray = Output.GetCharArray();
	CharArray.AddUninitialized(sizeof(FIoHash::ByteArray) * 2 + 1);
	UE::String::BytesToHexLower(Hash.GetBytes(), CharArray.GetData());
	CharArray.Last() = TEXT('\0');
	return Output;
}

FIoHash FIoHashBuilder::HashBuffer(const FCompositeBuffer& Buffer)
{
	return FBlake3::HashBuffer(Buffer);
}
