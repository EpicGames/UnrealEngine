// Copyright Epic Games, Inc. All Rights Reserved.

#include "IO/IoStatus.h"
#include "HAL/Platform.h"
#include "HAL/PlatformString.h"
#include "HAL/UnrealMemory.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Experimental/UnifiedError/UnifiedError.h"
#include "Experimental/IO/IoStatusError.h"
#include "Math/UnrealMathUtility.h"

DEFINE_LOG_CATEGORY_STATIC(LogIoStatus, Log, All);

//////////////////////////////////////////////////////////////////////////

const FIoStatus FIoStatus::Ok		{ EIoErrorCode::Ok,				TEXT("OK")				};
const FIoStatus FIoStatus::Unknown	{ EIoErrorCode::Unknown,		TEXT("Unknown Status")	};
const FIoStatus FIoStatus::Invalid	{ EIoErrorCode::InvalidCode,	TEXT("Invalid Code")	};

extern const TCHAR* const* GetIoErrorText_ErrorCodeText;

const TCHAR* GetIoErrorText(EIoErrorCode ErrorCode)
{
	if (ErrorCode >= EIoErrorCode::Last)
	{
		return GetIoErrorText_ErrorCodeText[static_cast<uint32>(EIoErrorCode::Unknown)];
	}
	return GetIoErrorText_ErrorCodeText[static_cast<uint32>(ErrorCode)];
}

//////////////////////////////////////////////////////////////////////////

FIoStatus::FIoStatus()
{
}

FIoStatus::~FIoStatus()
{
}

FIoStatus::FIoStatus(EIoErrorCode InErrorCode)
	: ErrorCode(InErrorCode)
{
}

FIoStatus::FIoStatus(EIoErrorCode InErrorCode, uint32 InSystemErrorCode)
	: ErrorCode(InErrorCode)
	, SystemErrorCode(InSystemErrorCode)
{
}

FIoStatus::FIoStatus(EIoErrorCode InErrorCode, const FStringView& InErrorMessage)
	: ErrorMessage(InErrorMessage)
	, ErrorCode(InErrorCode)
{
}

FIoStatus::FIoStatus(EIoErrorCode InErrorCode, uint32 InSystemErrorCode, const FStringView& InErrorMessage)
	: ErrorMessage(InErrorMessage)
	, ErrorCode(InErrorCode)
	, SystemErrorCode(InSystemErrorCode)
{
}

FIoStatus& FIoStatus::operator=(const FIoStatus& Other)
{
	ErrorMessage = Other.ErrorMessage;
	ErrorCode = Other.ErrorCode;
	SystemErrorCode	= Other.SystemErrorCode;

	return *this;
}

FIoStatus& FIoStatus::operator=(const EIoErrorCode InErrorCode)
{
	ErrorMessage.Reset();
	ErrorCode = InErrorCode;
	SystemErrorCode = 0;

	return *this;
}

bool FIoStatus::operator==(const FIoStatus& Other) const
{
	return ErrorMessage == Other.ErrorMessage && ErrorCode == Other.ErrorCode && SystemErrorCode == Other.SystemErrorCode;
}

FString FIoStatus::ToString() const
{
	if (ErrorMessage.IsEmpty())
	{
		if (LIKELY(SystemErrorCode == 0))
		{
			return FString::Format(TEXT("({0})"), { GetIoErrorText(ErrorCode) });
		}
		return FString::Format(TEXT("({0}, SystemErrorCode={1})"), { GetIoErrorText(ErrorCode), SystemErrorCode });
	}

	if (LIKELY(SystemErrorCode == 0))
	{
		return FString::Format(TEXT("{0} ({1})"), { *ErrorMessage, GetIoErrorText(ErrorCode) });
	}
	return FString::Format(TEXT("{0} ({1}, SystemErrorCode={2})"), { *ErrorMessage, GetIoErrorText(ErrorCode), SystemErrorCode });
}

void SerializeForLog(FCbWriter& Writer, const FIoStatus& Status)
{
	Writer.BeginObject();

	Writer.AddString(ANSITEXTVIEW("$type"), ANSITEXTVIEW("IoStatus"));
	// This probably could be optimized to avoid the FString creation
	Writer.AddString(ANSITEXTVIEW("$text"), Status.ToString());
	
	Writer.AddInteger(ANSITEXTVIEW("ErrorCode"), static_cast<uint32>(Status.ErrorCode));
	Writer.AddInteger(ANSITEXTVIEW("SystemErrorCode"), Status.SystemErrorCode);
	Writer.AddString(ANSITEXTVIEW("ErrorMessage"), *Status.ErrorMessage);

	Writer.EndObject();
}

void StatusOrCrash(const FIoStatus& Status)
{
	UE_LOGF(LogIoStatus, Fatal, "I/O Error '%ls'", *Status.ToString());
}

//////////////////////////////////////////////////////////////////////////

FIoStatusBuilder::FIoStatusBuilder(EIoErrorCode InStatusCode)
	: StatusCode(InStatusCode)
	, SystemErrorCode(0)
{
}

FIoStatusBuilder::FIoStatusBuilder(EIoErrorCode InStatusCode, uint32 InSystemErrorCode)
	: StatusCode(InStatusCode)
	, SystemErrorCode(InSystemErrorCode)
{
}

FIoStatusBuilder::~FIoStatusBuilder()
{
}

FIoStatusBuilder::operator FIoStatus()
{
	return FIoStatus(StatusCode, SystemErrorCode, Message);
}

FIoStatusBuilder& FIoStatusBuilder::operator<<(FStringView String)
{
	Message.Append(String);

	return *this;
}
