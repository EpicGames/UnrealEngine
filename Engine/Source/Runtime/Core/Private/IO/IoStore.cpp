// Copyright Epic Games, Inc. All Rights Reserved.

#include "IO/IoStore.h"

#include "Async/MappedFileHandle.h"
#include "Containers/Map.h"
#include "Features/IModularFeatures.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"
#include "IO/IoDirectoryIndex.h"
#include "Misc/Compression.h"
#include "Misc/CoreDelegates.h"
#include "Misc/Paths.h"
#include "Misc/StringBuilder.h"
#include "ProfilingDebugging/CountersTrace.h"
#include "Serialization/BufferWriter.h"
#include "Serialization/LargeMemoryReader.h"
#include "Tasks/Task.h"
#include "Templates/UniquePtr.h"

DEFINE_LOG_CATEGORY(LogIoStore);

//////////////////////////////////////////////////////////////////////////

//TRACE_DECLARE_INT_COUNTER(IoStoreAvailableCompressionBuffers, TEXT("IoStore/AvailableCompressionBuffers"));

template<typename ArrayType>
bool WriteArray(IFileHandle* FileHandle, const ArrayType& Array)
{
	return FileHandle->Write(reinterpret_cast<const uint8*>(Array.GetData()), Array.GetTypeSize() * Array.Num());
}

static IEngineCrypto* GetEngineCrypto()
{
	static bool bFeaturesInitialized = false;
	static TArray<IEngineCrypto*> Features;
	if (!bFeaturesInitialized)
	{
		IModularFeatures::FScopedLockModularFeatureList ScopedLockModularFeatureList;
		// bFeaturesInitialized is not atomic so it can have potentially become true since we last checked (except now we're under a lock: IModularFeatures::FScopedLockModularFeatureList, so it will be fine now):
		if (!bFeaturesInitialized)
		{
		    Features = IModularFeatures::Get().GetModularFeatureImplementations<IEngineCrypto>(IEngineCrypto::GetFeatureName());
		    bFeaturesInitialized = true;
		}
	}
	checkf(Features.Num() > 0, TEXT("RSA functionality was used but no modular feature was registered to provide it. Please make sure your project has the PlatformCrypto plugin enabled!"));
	return Features[0];
}

static bool IsSigningEnabled()
{
#if UE_BUILD_SHIPPING
	return FCoreDelegates::GetPakSigningKeysDelegate().IsBound();
#else
	return false;
#endif
}

static FRSAKeyHandle GetPublicSigningKey()
{
	static FRSAKeyHandle PublicKey = InvalidRSAKeyHandle;
	static bool bInitializedPublicKey = false;
	if (!bInitializedPublicKey)
	{
		TDelegate<void(TArray<uint8>&, TArray<uint8>&)>& Delegate = FCoreDelegates::GetPakSigningKeysDelegate();
		if (Delegate.IsBound())
		{
			TArray<uint8> Exponent;
			TArray<uint8> Modulus;
			Delegate.Execute(Exponent, Modulus);
			PublicKey = GetEngineCrypto()->CreateRSAKey(Exponent, TArray<uint8>(), Modulus);
		}
		bInitializedPublicKey = true;
	}

	return PublicKey;
}

static FIoStatus CreateContainerSignature(
	const FRSAKeyHandle PrivateKey,
	const FIoStoreTocHeader& TocHeader,
	TArrayView<const FSHAHash> BlockSignatureHashes,
	TArray<uint8>& OutTocSignature,
	TArray<uint8>& OutBlockSignature)
{
	if (PrivateKey == InvalidRSAKeyHandle)
	{
		return FIoStatus(EIoErrorCode::SignatureError, TEXT("Invalid signing key"));
	}

	FSHAHash TocHash, BlocksHash;

	FSHA1::HashBuffer(reinterpret_cast<const uint8*>(&TocHeader), sizeof(FIoStoreTocHeader), TocHash.Hash);
	FSHA1::HashBuffer(BlockSignatureHashes.GetData(), BlockSignatureHashes.Num() * sizeof(FSHAHash), BlocksHash.Hash);

	int32 BytesEncrypted = GetEngineCrypto()->EncryptPrivate(TArrayView<const uint8>(TocHash.Hash, UE_ARRAY_COUNT(FSHAHash::Hash)), OutTocSignature, PrivateKey);

	if (BytesEncrypted < 1)
	{
		return FIoStatus(EIoErrorCode::SignatureError, TEXT("Failed to encrypt TOC signature"));
	}

	BytesEncrypted = GetEngineCrypto()->EncryptPrivate(TArrayView<const uint8>(BlocksHash.Hash, UE_ARRAY_COUNT(FSHAHash::Hash)), OutBlockSignature, PrivateKey);

	return BytesEncrypted > 0 ? FIoStatus::Ok : FIoStatus(EIoErrorCode::SignatureError, TEXT("Failed to encrypt block signature"));
}

static FIoStatus ValidateContainerSignature(
	const FRSAKeyHandle PublicKey,
	const FIoStoreTocHeader& TocHeader,
	TArrayView<const FSHAHash> BlockSignatureHashes,
	TArrayView<const uint8> TocSignature,
	TArrayView<const uint8> BlockSignature)
{
	if (PublicKey == InvalidRSAKeyHandle)
	{
		return FIoStatus(EIoErrorCode::SignatureError, TEXT("Invalid signing key"));
	}

	TArray<uint8> DecryptedTocHash, DecryptedBlocksHash;

	int32 BytesDecrypted = GetEngineCrypto()->DecryptPublic(TocSignature, DecryptedTocHash, PublicKey);
	if (BytesDecrypted != UE_ARRAY_COUNT(FSHAHash::Hash))
	{
		return FIoStatus(EIoErrorCode::SignatureError, TEXT("Failed to decrypt TOC signature"));
	}

	BytesDecrypted = GetEngineCrypto()->DecryptPublic(BlockSignature, DecryptedBlocksHash, PublicKey);
	if (BytesDecrypted != UE_ARRAY_COUNT(FSHAHash::Hash))
	{
		return FIoStatus(EIoErrorCode::SignatureError, TEXT("Failed to decrypt block signature"));
	}

	FSHAHash TocHash, BlocksHash;
	FSHA1::HashBuffer(reinterpret_cast<const uint8*>(&TocHeader), sizeof(FIoStoreTocHeader), TocHash.Hash);
	FSHA1::HashBuffer(BlockSignatureHashes.GetData(), BlockSignatureHashes.Num() * sizeof(FSHAHash), BlocksHash.Hash);

	if (FMemory::Memcmp(DecryptedTocHash.GetData(), TocHash.Hash, DecryptedTocHash.Num()) != 0)
	{
		return FIoStatus(EIoErrorCode::SignatureError, TEXT("Invalid TOC signature"));
	}

	if (FMemory::Memcmp(DecryptedBlocksHash.GetData(), BlocksHash.Hash, DecryptedBlocksHash.Num()) != 0)
	{
		return FIoStatus(EIoErrorCode::SignatureError, TEXT("Invalid block signature"));
	}

	return FIoStatus::Ok;
}

class FIoStoreTocReader : public IIoStoreTocReader
{
public:
	FIoStoreTocReader()
	{
		FMemory::Memzero(&Toc.Header, sizeof(FIoStoreTocHeader));
	}
	virtual ~FIoStoreTocReader() = default;

	[[nodiscard]] FIoStatus Read(const TCHAR* TocFilePath, const TMap<FGuid, FAES::FAESKey>& DecryptionKeys)
	{
		FIoStatus TocStatus = FIoStoreTocResourceView::Read(TocFilePath, EIoStoreTocReadOptions::ReadAll, Toc, TocStorage);
		if (!TocStatus.IsOk())
		{
			return TocStatus;
		}

		ChunkIdToIndex.Empty(Toc.ChunkIds.Num());

		for (int32 ChunkIndex = 0; ChunkIndex < Toc.ChunkIds.Num(); ++ChunkIndex)
		{
			ChunkIdToIndex.Add(Toc.ChunkIds[ChunkIndex], ChunkIndex);
		}

		if (EnumHasAnyFlags(Toc.Header.ContainerFlags, EIoContainerFlags::Encrypted))
		{
			const FAES::FAESKey* FindKey = DecryptionKeys.Find(Toc.Header.EncryptionKeyGuid);
			if (!FindKey)
			{
				return FIoStatusBuilder(EIoErrorCode::FileOpenFailed) << TEXT("Missing decryption key for IoStore container file '") << TocFilePath << TEXT("'");
			}
			DecryptionKey = *FindKey;
		}

		if (EnumHasAnyFlags(Toc.Header.ContainerFlags, EIoContainerFlags::Indexed) &&
			Toc.DirectoryIndexBuffer.Num() > 0)
		{
			FIoStatus DirectoryIndexStatus = DirectoryIndexReader.Initialize(Toc.DirectoryIndexBuffer, DecryptionKey);
			if (!DirectoryIndexStatus.IsOk())
			{
				return DirectoryIndexStatus;
			}
			DirectoryIndexReader.IterateDirectoryIndex(
				FIoDirectoryIndexHandle::RootDirectory(),
				TEXT(""),
				[this](FStringView Filename, uint32 TocEntryIndex) -> bool
				{
					AddFileName(TocEntryIndex, Filename);
					return true;
				});
		}

		return TocStatus;
	}


	const FIoStoreTocResourceView& GetTocResource() const
	{
		return Toc;
	}

	const FAES::FAESKey& GetDecryptionKey() const
	{
		return DecryptionKey;
	}

	const FIoDirectoryIndexReader& GetDirectoryIndexReader() const
	{
		return DirectoryIndexReader;
	}

	const int32* GetTocEntryIndex(const FIoChunkId& ChunkId) const
	{
		return ChunkIdToIndex.Find(ChunkId);
	}

	const FIoOffsetAndLength* GetOffsetAndLength(const FIoChunkId& ChunkId) const
	{
		if (const int32* Index = ChunkIdToIndex.Find(ChunkId))
		{
			return &Toc.ChunkOffsetLengths[*Index];
		}

		return nullptr;
	}

	FIoStoreTocChunkInfo GetTocChunkInfo(int32 TocEntryIndex) const
	{
		FIoStoreTocChunkInfo ChunkInfo = Toc.GetTocChunkInfo(TocEntryIndex);

		if (const FString* FileName = IndexToFileName.Find(TocEntryIndex); FileName != nullptr)
		{
			ChunkInfo.FileName = *FileName;
			ChunkInfo.bHasValidFileName = true;
		}
		else
		{
			ChunkInfo.FileName = FString::Printf(TEXT("<%s>"), *LexToString(ChunkInfo.ChunkType));
			ChunkInfo.bHasValidFileName = false;
		}
		return ChunkInfo;
	}

private:
	void AddFileName(int32 TocEntryIndex, FStringView Filename)
	{
		IndexToFileName.Add(TocEntryIndex, FString(Filename));
	}

	FIoStoreTocResourceView Toc;
	FIoStoreTocResourceStorage TocStorage;
	FIoDirectoryIndexReader DirectoryIndexReader;
	FAES::FAESKey DecryptionKey;
	TMap<FIoChunkId, int32> ChunkIdToIndex;
	TMap<int32, FString> IndexToFileName;
};

IIoStoreTocReader::~IIoStoreTocReader() = default;

/* static */ TIoStatusOr<TUniquePtr<IIoStoreTocReader>> IIoStoreTocReader::ReadFromDisk(const TCHAR* TocFilePath, const TMap<FGuid, FAES::FAESKey>& DecryptionKeys)
{
	TUniquePtr<IIoStoreTocReader> TocReader = MakeUnique<FIoStoreTocReader>();
	FIoStatus Result = ((FIoStoreTocReader*)TocReader.Get())->Read(TocFilePath, DecryptionKeys);
	if (Result != FIoStatus::Ok)
	{
		return Result;
	}
	return TocReader;
}

template<typename T>
static FIoStoreTocChunkInfo GetTocChunkInfoInternal(const T& Toc, int32 TocEntryIndex)
{
	const FIoStoreTocEntryMeta& Meta = Toc.ChunkMetas[TocEntryIndex];
	const FIoOffsetAndLength& OffsetLength = Toc.ChunkOffsetLengths[TocEntryIndex];

	const bool bIsContainerCompressed = EnumHasAnyFlags(Toc.Header.ContainerFlags, EIoContainerFlags::Compressed);

	FIoStoreTocChunkInfo ChunkInfo;
	ChunkInfo.Id = Toc.ChunkIds[TocEntryIndex];
	ChunkInfo.ChunkType = ChunkInfo.Id.GetChunkType();
	ChunkInfo.ChunkHash = Meta.ChunkHash;
	ChunkInfo.bHasValidFileName = false;
	ChunkInfo.bIsCompressed = EnumHasAnyFlags(Meta.Flags, FIoStoreTocEntryMetaFlags::Compressed);
	ChunkInfo.bIsMemoryMapped = EnumHasAnyFlags(Meta.Flags, FIoStoreTocEntryMetaFlags::MemoryMapped);
	ChunkInfo.bForceUncompressed = bIsContainerCompressed && !EnumHasAnyFlags(Meta.Flags, FIoStoreTocEntryMetaFlags::Compressed);
	ChunkInfo.Offset = OffsetLength.GetOffset();
	ChunkInfo.Size = OffsetLength.GetLength();

	const uint64 CompressionBlockSize = Toc.Header.CompressionBlockSize;
	int32 FirstBlockIndex = int32(ChunkInfo.Offset / CompressionBlockSize);
	int32 LastBlockIndex = int32((Align(ChunkInfo.Offset + ChunkInfo.Size, CompressionBlockSize) - 1) / CompressionBlockSize);

	ChunkInfo.NumCompressedBlocks = LastBlockIndex - FirstBlockIndex + 1;
	ChunkInfo.OffsetOnDisk = Toc.CompressionBlocks[FirstBlockIndex].GetOffset();
	ChunkInfo.CompressedSize = 0;
	ChunkInfo.PartitionIndex = -1;
	ChunkInfo.SizeOnDisk = 0;
	for (int32 BlockIndex = FirstBlockIndex; BlockIndex <= LastBlockIndex; ++BlockIndex)
	{
		const FIoStoreTocCompressedBlockEntry& CompressionBlock = Toc.CompressionBlocks[BlockIndex];
		ChunkInfo.CompressedSize += CompressionBlock.GetCompressedSize();
		ChunkInfo.SizeOnDisk += Align(CompressionBlock.GetCompressedSize(), FAES::AESBlockSize);
		if (ChunkInfo.PartitionIndex < 0)
		{
			ChunkInfo.PartitionIndex = int32(CompressionBlock.GetOffset() / Toc.Header.PartitionSize);
		}
	}
	return ChunkInfo;
}

FIoStoreTocChunkInfo FIoStoreTocResourceView::GetTocChunkInfo(int32 TocEntryIndex) const
{
	return GetTocChunkInfoInternal(*this, TocEntryIndex);
}

class FIoStoreReaderImpl
{
public:
	FIoStoreReaderImpl()
	{

	}

	//
	// GenericPlatformFile isn't designed around a lot of jobs throwing accesses at it, so instead we 
	// use IFileHandle directly and round robin between a number of file handles in order to saturate
	// year 2022 ssd drives. For a file hot in the windows file cache, you can get 4+ GB/s with as few as 
	// 4 file handles, however for a cold file you need upwards of 32 in order to reach ~1.5 GB/s. This is
	// low because IoStoreReader (note: not IoDispatcher!) reads are comparatively small - at most you're reading compression block sized
	// chunks when uncompressed, however with Oodle those get cut by ~half, so with a default block size
	// of 64kb, reads are generally less than 32kb, which is tough to use and get full ssd bandwidth out of.
	//
	static constexpr uint32 NumHandlesPerFile = 12;
	struct FContainerFileAccess
	{
		FCriticalSection HandleLock[NumHandlesPerFile];
		IFileHandle* Handle[NumHandlesPerFile];
		std::atomic_uint32_t NextHandleIndex{0};
		bool bValid = false;

		FContainerFileAccess(IPlatformFile& Ipf, const TCHAR* ContainerFileName)
		{
			bValid = true;
			for (uint32 i=0; i < NumHandlesPerFile; i++)
			{
				Handle[i] = Ipf.OpenRead(ContainerFileName);
				if (Handle[i] == nullptr)
				{
					bValid = false;
				}
			}
		}

		~FContainerFileAccess()
		{
			for (int32 Index = 0; Index < NumHandlesPerFile; Index++)
			{
				if (Handle[Index] != nullptr)
				{
					delete Handle[Index];
					Handle[Index] = nullptr;
				}
			}
		}

		bool IsValid() const { return bValid; }
	};
	

	// Kick off an async read from the iostore container, rotating between the file handles for the partition.
	UE::Tasks::FTask StartAsyncRead(int32 InPartitionIndex, int64 InPartitionOffset, int64 InReadAmount, uint8* OutBuffer, std::atomic_bool* OutSuccess) const
	{
		return UE::Tasks::Launch(TEXT("FIoStoreReader_AsyncRead"), [this, InPartitionIndex, InPartitionOffset, OutBuffer, InReadAmount, OutSuccess]() mutable
		{
			FContainerFileAccess* ContainerFileAccess = this->ContainerFileAccessors[InPartitionIndex].Get();

			// Round robin between the file handles. Since we are always reading blocks, everything is ~roughly~ the same
			// size so we don't have to worry about a single huge read backing up one handle.
			uint32 OurIndex = ContainerFileAccess->NextHandleIndex.fetch_add(1);
			OurIndex %= NumHandlesPerFile;

			// Each file handle can only be touched by one task at a time. We use an OS lock so that the OS scheduler
			// knows we're in a wait state and who we're waiting on.
			//
			// CAUTION if any overload of IFileHandle launches tasks (... unlikely ...) this could deadlock if NumHandlesPerFile is more
			// than the number of worker threads, as the OS lock will not do task retraction.
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(FIoStoreReader_StartAsyncRead_Lock);
				ContainerFileAccess->HandleLock[OurIndex].Lock();
			}

			bool bReadSucceeded;
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(FIoStoreReader_StartAsyncRead_SeekAndRead);
				ContainerFileAccess->Handle[OurIndex]->Seek(InPartitionOffset);
				bReadSucceeded = ContainerFileAccess->Handle[OurIndex]->Read(OutBuffer, InReadAmount);
			}

			OutSuccess->store(bReadSucceeded);
			ContainerFileAccess->HandleLock[OurIndex].Unlock();
		});
	}

	[[nodiscard]] FIoStatus Initialize(FStringView InContainerPath, const TMap<FGuid, FAES::FAESKey>& InDecryptionKeys)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FIoStoreReader::Initialize);
		ContainerPath = InContainerPath;

		TStringBuilder<256> TocFilePath;
		TocFilePath.Append(InContainerPath);
		TocFilePath.Append(TEXT(".utoc"));

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FIoStoreReader::ReadToc);
			FIoStatus TocStatus = TocReader.Read(*TocFilePath, InDecryptionKeys);
			if (!TocStatus.IsOk())
			{
				return TocStatus;
			}
		}

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FIoStoreReader::GenerateContainerFileAccessors);
			const FIoStoreTocResourceView& TocResource = TocReader.GetTocResource();

			IPlatformFile& Ipf = FPlatformFileManager::Get().GetPlatformFile();
			ContainerFileAccessors.Reserve(TocResource.Header.PartitionCount);
			for (uint32 PartitionIndex = 0; PartitionIndex < TocResource.Header.PartitionCount; ++PartitionIndex)
			{
				TStringBuilder<256> ContainerFilePath;
				ContainerFilePath.Append(InContainerPath);
				if (PartitionIndex > 0)
				{
					ContainerFilePath.Append(FString::Printf(TEXT("_s%d"), PartitionIndex));
				}
				ContainerFilePath.Append(TEXT(".ucas"));

				ContainerFileAccessors.Emplace(new FContainerFileAccess(Ipf, *ContainerFilePath));
				if (ContainerFileAccessors.Last().IsValid() == false)
				{
					return FIoStatusBuilder(EIoErrorCode::FileOpenFailed) << TEXT("Failed to open IoStore container file '") << *TocFilePath << TEXT("'");
				}
			}
		}

		return FIoStatus::Ok;
	}

	FIoContainerId GetContainerId() const
	{
		return TocReader.GetTocResource().Header.ContainerId;
	}

	uint32 GetVersion() const
	{
		return TocReader.GetTocResource().Header.Version;
	}

	EIoContainerFlags GetContainerFlags() const
	{
		return TocReader.GetTocResource().Header.ContainerFlags;
	}

	FGuid GetEncryptionKeyGuid() const
	{
		return TocReader.GetTocResource().Header.EncryptionKeyGuid;
	}

	FString GetContainerName() const
	{
		return FPaths::GetBaseFilename(ContainerPath);
	}

	int32 GetChunkCount() const 
	{
		return TocReader.GetTocResource().ChunkIds.Num();
	}

	void EnumerateChunks(TFunction<bool(FIoStoreTocChunkInfo&&)>&& Callback) const
	{
		const FIoStoreTocResourceView& TocResource = TocReader.GetTocResource();

		for (int32 ChunkIndex = 0; ChunkIndex < TocResource.ChunkIds.Num(); ++ChunkIndex)
		{
			FIoStoreTocChunkInfo ChunkInfo = TocReader.GetTocChunkInfo(ChunkIndex);
			if (!Callback(MoveTemp(ChunkInfo)))
			{
				break;
			}
		}
	}

	TIoStatusOr<FIoStoreTocChunkInfo> GetChunkInfo(const FIoChunkId& ChunkId) const
	{
		const int32* TocEntryIndex = TocReader.GetTocEntryIndex(ChunkId);
		if (TocEntryIndex)
		{
			return TocReader.GetTocChunkInfo(*TocEntryIndex);
		}
		else
		{
			return FIoStatus(EIoErrorCode::UnknownChunkID, FString::Printf(TEXT("Unknown chunk ID '%s'"), *LexToString(ChunkId)));
		}
	}

	TIoStatusOr<FIoStoreTocChunkInfo> GetChunkInfo(const uint32 TocEntryIndex) const
	{
		const FIoStoreTocResourceView& TocResource = TocReader.GetTocResource();

		if (TocEntryIndex < uint32(TocResource.ChunkIds.Num()))
		{
			return TocReader.GetTocChunkInfo(TocEntryIndex);
		}
		else
		{
			return FIoStatus(EIoErrorCode::InvalidParameter, TEXT("Invalid TocEntryIndex"));
		}
	}

	UE::Tasks::TTask<TIoStatusOr<FIoBuffer>> ReadAsync(const FIoChunkId& ChunkId, const FIoReadOptions& Options) const
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ReadChunkAsync);

		struct FState
		{
			TArray64<uint8> CompressedBuffer;
			uint64 CompressedSize = 0;
			uint64 UncompressedSize = 0;
			TOptional<FIoBuffer> UncompressedBuffer;
			std::atomic_bool bReadSucceeded {false};
			std::atomic_bool bUncompressFailed { false };
		};

		const FIoOffsetAndLength* OffsetAndLength = TocReader.GetOffsetAndLength(ChunkId);
		if (!OffsetAndLength )
		{
			// Currently there's no way to make a task with a valid result that just emplaces
			// without running.
			return UE::Tasks::Launch(TEXT("FIoStoreRead_Error"), 
				[ChunkId] { return TIoStatusOr<FIoBuffer>(FIoStatus(EIoErrorCode::UnknownChunkID, FString::Printf(TEXT("Unknown chunk ID '%s'"), *LexToString(ChunkId)))); },
				UE::Tasks::ETaskPriority::Normal,
				UE::Tasks::EExtendedTaskPriority::Inline); // force execution on this thread
		}

		const uint64 RequestedOffset = Options.GetOffset();
		const uint64 ResolvedOffset = OffsetAndLength->GetOffset() + RequestedOffset;
		const uint64 ResolvedSize = RequestedOffset <= OffsetAndLength->GetLength() ? FMath::Min(Options.GetSize(), OffsetAndLength->GetLength() - RequestedOffset) : 0;
		const FIoStoreTocResourceView& TocResource = TocReader.GetTocResource();
		const uint64 CompressionBlockSize = TocResource.Header.CompressionBlockSize;
		const int32 FirstBlockIndex = int32(ResolvedOffset / CompressionBlockSize);
		const int32 LastBlockIndex = int32((Align(ResolvedOffset + ResolvedSize, CompressionBlockSize) - 1) / CompressionBlockSize);
		const int32 BlockCount = LastBlockIndex - FirstBlockIndex + 1;
		if (!BlockCount)
		{
			// Currently there's no way to make a task with a valid result that just emplaces
			// without running.
			return UE::Tasks::Launch(TEXT("FIoStoreRead_Empty"),
				[] { return TIoStatusOr<FIoBuffer>(); },
				UE::Tasks::ETaskPriority::Normal,
				UE::Tasks::EExtendedTaskPriority::Inline); // force execution on this thread
		}
		const FIoStoreTocCompressedBlockEntry& FirstBlock = TocResource.CompressionBlocks[FirstBlockIndex];
		const FIoStoreTocCompressedBlockEntry& LastBlock = TocResource.CompressionBlocks[LastBlockIndex];
		const int32 PartitionIndex = static_cast<int32>(FirstBlock.GetOffset() / TocResource.Header.PartitionSize);
		check(static_cast<int32>(LastBlock.GetOffset() / TocResource.Header.PartitionSize) == PartitionIndex);
		const uint64 ReadStartOffset = FirstBlock.GetOffset() % TocResource.Header.PartitionSize;
		const uint64 ReadEndOffset = (LastBlock.GetOffset() + Align(LastBlock.GetCompressedSize(), FAES::AESBlockSize)) % TocResource.Header.PartitionSize;
		FState* State = new FState();
		State->CompressedSize = ReadEndOffset - ReadStartOffset;
		State->UncompressedSize = ResolvedSize;
		State->CompressedBuffer.AddUninitialized(State->CompressedSize);
		State->UncompressedBuffer.Emplace(State->UncompressedSize);

		UE::Tasks::FTask ReadJob = StartAsyncRead(PartitionIndex, ReadStartOffset, (int32)State->CompressedSize, State->CompressedBuffer.GetData(), &State->bReadSucceeded);

		UE::Tasks::TTask<TIoStatusOr<FIoBuffer>> ReturnTask = UE::Tasks::Launch(TEXT("FIoStoreReader::AsyncRead"), [this, State, PartitionIndex, CompressionBlockSize, ResolvedOffset, FirstBlockIndex, LastBlockIndex, ResolvedSize, ReadStartOffset, &TocResource]()
		{			
			UE::Tasks::FTaskEvent DecompressionDoneEvent(TEXT("FIoStoreReader::DecompressionDone"));

			uint64 CompressedSourceOffset = 0;
			uint64 UncompressedDestinationOffset = 0;
			uint64 OffsetInBlock = ResolvedOffset % CompressionBlockSize;
			uint64 RemainingSize = ResolvedSize;
			for (int32 BlockIndex = FirstBlockIndex; BlockIndex <= LastBlockIndex; ++BlockIndex)
			{
				UE::Tasks::FTask DecompressBlockTask = UE::Tasks::Launch(TEXT("FIoStoreReader::Decompress"), [this, State, BlockIndex, CompressedSourceOffset, UncompressedDestinationOffset, OffsetInBlock, RemainingSize]()
				{
					if (State->bReadSucceeded)
					{
						uint8* CompressedSource = State->CompressedBuffer.GetData() + CompressedSourceOffset;
						uint8* UncompressedDestination = State->UncompressedBuffer->Data() + UncompressedDestinationOffset;
						const FIoStoreTocResourceView& TocResource = TocReader.GetTocResource();
						const FIoStoreTocCompressedBlockEntry& CompressionBlock = TocResource.CompressionBlocks[BlockIndex];
						const uint32 RawSize = Align(CompressionBlock.GetCompressedSize(), FAES::AESBlockSize);
						const uint32 UncompressedSize = CompressionBlock.GetUncompressedSize();
						FName CompressionMethod = TocResource.CompressionMethods[CompressionBlock.GetCompressionMethodIndex()];
						if (EnumHasAnyFlags(TocResource.Header.ContainerFlags, EIoContainerFlags::Encrypted))
						{
							TRACE_CPUPROFILER_EVENT_SCOPE(Decrypt);
							check(CompressedSource + RawSize <= State->CompressedBuffer.GetData() + State->CompressedSize);
							FAES::DecryptData(CompressedSource, RawSize, TocReader.GetDecryptionKey());
						}
						if (CompressionMethod.IsNone())
						{
							check(UncompressedDestination + UncompressedSize - OffsetInBlock <= State->UncompressedBuffer->Data() + State->UncompressedBuffer->DataSize());
							FMemory::Memcpy(UncompressedDestination, CompressedSource + OffsetInBlock, UncompressedSize - OffsetInBlock);
						}
						else
						{
							bool bUncompressed;
							if (OffsetInBlock || RemainingSize < UncompressedSize)
							{
								TArray<uint8> TempBuffer;
								TempBuffer.SetNumUninitialized(UncompressedSize);
								bUncompressed = FCompression::UncompressMemory(CompressionMethod, TempBuffer.GetData(), UncompressedSize, CompressedSource, CompressionBlock.GetCompressedSize());
								uint64 CopySize = FMath::Min<uint64>(UncompressedSize - OffsetInBlock, RemainingSize);
								FMemory::Memcpy(UncompressedDestination, TempBuffer.GetData() + OffsetInBlock, CopySize);
							}
							else
							{
								check(UncompressedDestination + UncompressedSize <= State->UncompressedBuffer->Data() + State->UncompressedBuffer->DataSize());
								bUncompressed = FCompression::UncompressMemory(CompressionMethod, UncompressedDestination, UncompressedSize, CompressedSource, CompressionBlock.GetCompressedSize());
							}
							if (!bUncompressed)
							{
								State->bUncompressFailed = true;
							}
						}
					} // end if read succeeded

				}); // end decompression lambda

				DecompressionDoneEvent.AddPrerequisites(DecompressBlockTask);

				const FIoStoreTocCompressedBlockEntry& CompressionBlock = TocResource.CompressionBlocks[BlockIndex];
				const uint32 RawSize = Align(CompressionBlock.GetCompressedSize(), FAES::AESBlockSize);
				CompressedSourceOffset += RawSize;
				UncompressedDestinationOffset += CompressionBlock.GetUncompressedSize();
				RemainingSize -= CompressionBlock.GetUncompressedSize();
				OffsetInBlock = 0;
			} // end for each block

			// Unlock the event so we're now only waiting on the prerequisites
			DecompressionDoneEvent.Trigger();
			// Wait for everything and potentially help with the decompression tasks by retraction.
			DecompressionDoneEvent.Wait();

			TIoStatusOr<FIoBuffer> Result;
			if (State->bReadSucceeded == false)
			{
				Result = FIoStatus(EIoErrorCode::ReadError, TEXT("Failed reading chunk from container file"));
			}
			else if (State->bUncompressFailed)
			{
				Result = FIoStatus(EIoErrorCode::ReadError, TEXT("Failed uncompressing chunk"));
			}
			else
			{
				Result = State->UncompressedBuffer.GetValue();
			}
			delete State;

			return Result;
		}, UE::Tasks::Prerequisites(ReadJob)); // end read and compress lambda launch

		return ReturnTask;
	} // end ReadAsync

	TIoStatusOr<FIoBuffer> Read(const FIoChunkId& ChunkId, const FIoReadOptions& Options) const
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ReadChunk);

		const FIoOffsetAndLength* OffsetAndLength = TocReader.GetOffsetAndLength(ChunkId);
		if (!OffsetAndLength)
		{
			return FIoStatus(EIoErrorCode::UnknownChunkID, FString::Printf(TEXT("Unknown chunk ID '%s'"), *LexToString(ChunkId)));
		}

		uint64 RequestedOffset = Options.GetOffset();
		uint64 ResolvedOffset = OffsetAndLength->GetOffset() + RequestedOffset;
		uint64 ResolvedSize = 0;
		if (RequestedOffset <= OffsetAndLength->GetLength())
		{
			ResolvedSize = FMath::Min(Options.GetSize(), OffsetAndLength->GetLength() - RequestedOffset);
		}

		const FIoStoreTocResourceView& TocResource = TocReader.GetTocResource();
		const uint64 CompressionBlockSize = TocResource.Header.CompressionBlockSize;
		FIoBuffer UncompressedBuffer(ResolvedSize);
		if (ResolvedSize == 0)
		{
			return UncompressedBuffer;
		}

		// From here on we are reading / decompressing at least one block.

		// We try to overlap the IO for the next block with the decrypt/decompress for the current
		// block, which requires two IO buffers:
		TArray<uint8> CompressedBuffers[2];
		std::atomic_bool AsyncReadSucceeded[2];

		int32 FirstBlockIndex = int32(ResolvedOffset / CompressionBlockSize);
		int32 LastBlockIndex = int32((Align(ResolvedOffset + ResolvedSize, CompressionBlockSize) - 1) / CompressionBlockSize);

		// Lambda to kick off a read with a sufficient output buffer.
		auto LaunchBlockRead = [&TocResource, this](int32 BlockIndex, TArray<uint8>& DestinationBuffer, std::atomic_bool* OutReadSucceeded)
		{
			const uint64 CompressionBlockSize = TocResource.Header.CompressionBlockSize;
			const FIoStoreTocCompressedBlockEntry& CompressionBlock = TocResource.CompressionBlocks[BlockIndex];

			// CompressionBlockSize is technically the _uncompresseed_ block size, however it's a good
			// size to use for reuse as block compression can vary wildly and we want to be able to
			// read blocks that happen to be uncompressed.
			uint32 SizeForDecrypt = Align(CompressionBlock.GetCompressedSize(), FAES::AESBlockSize);
			uint32 CompressedBufferSizeNeeded = FMath::Max(uint32(CompressionBlockSize), SizeForDecrypt);

			if (uint32(DestinationBuffer.Num()) < CompressedBufferSizeNeeded)
			{
				DestinationBuffer.SetNumUninitialized(CompressedBufferSizeNeeded);
			}

			int32 PartitionIndex = int32(CompressionBlock.GetOffset() / TocResource.Header.PartitionSize);
			int64 PartitionOffset = int64(CompressionBlock.GetOffset() % TocResource.Header.PartitionSize);
			return StartAsyncRead(PartitionIndex, PartitionOffset, SizeForDecrypt, DestinationBuffer.GetData(), OutReadSucceeded);
		};


		// Kick off the first async read
		UE::Tasks::FTask NextReadRequest;
		uint8 NextReadBufferIndex = 0;
		NextReadRequest = LaunchBlockRead(FirstBlockIndex, CompressedBuffers[NextReadBufferIndex], &AsyncReadSucceeded[NextReadBufferIndex]);

		uint64 UncompressedDestinationOffset = 0;
		uint64 OffsetInBlock = ResolvedOffset % CompressionBlockSize;
		uint64 RemainingSize = ResolvedSize;
		TArray<uint8> TempBuffer;
		for (int32 BlockIndex = FirstBlockIndex; BlockIndex <= LastBlockIndex; ++BlockIndex)
		{
			// Kick off the next block's IO if there is one
			UE::Tasks::FTask ReadRequest(MoveTemp(NextReadRequest));
			uint8 OurBufferIndex = NextReadBufferIndex;
			if (BlockIndex + 1 <= LastBlockIndex)
			{
				NextReadBufferIndex = NextReadBufferIndex ^ 1;
				NextReadRequest = LaunchBlockRead(BlockIndex + 1, CompressedBuffers[NextReadBufferIndex], &AsyncReadSucceeded[NextReadBufferIndex]);
			}

			// Now, wait for _our_ block's IO
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(WaitForIo);
				ReadRequest.Wait();
			}

			if (AsyncReadSucceeded[OurBufferIndex] == false)
			{
				return FIoStatus(EIoErrorCode::ReadError, TEXT("Failed async read in FIoStoreReader::ReadCompressed"));
			}

			const FIoStoreTocCompressedBlockEntry& CompressionBlock = TocResource.CompressionBlocks[BlockIndex];

			// This also happened in the LaunchBlockRead call, so we know the buffer has the necessary size.
			uint32 RawSize = Align(CompressionBlock.GetCompressedSize(), FAES::AESBlockSize);
			if (EnumHasAnyFlags(TocResource.Header.ContainerFlags, EIoContainerFlags::Encrypted))
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(Decrypt);
				FAES::DecryptData(CompressedBuffers[OurBufferIndex].GetData(), RawSize, TocReader.GetDecryptionKey());
			}

			FName CompressionMethod = TocResource.CompressionMethods[CompressionBlock.GetCompressionMethodIndex()];
			uint8* UncompressedDestination = UncompressedBuffer.Data() + UncompressedDestinationOffset;
			const uint32 UncompressedSize = CompressionBlock.GetUncompressedSize();
			if (CompressionMethod.IsNone())
			{
				uint64 CopySize = FMath::Min<uint64>(UncompressedSize - OffsetInBlock, RemainingSize);
				check(UncompressedDestination + CopySize <= UncompressedBuffer.Data() + UncompressedBuffer.DataSize());
				FMemory::Memcpy(UncompressedDestination, CompressedBuffers[OurBufferIndex].GetData() + OffsetInBlock, CopySize);
				UncompressedDestinationOffset += CopySize;
				RemainingSize -= CopySize;
			}
			else
			{
				bool bUncompressed;
				if (OffsetInBlock || RemainingSize < UncompressedSize)
				{
					// If this block is larger than the amount of data actually requested, decompress to a temp
					// buffer and then copy out. Should never happen when reading the entire chunk.
					TempBuffer.SetNumUninitialized(UncompressedSize);
					bUncompressed = FCompression::UncompressMemory(CompressionMethod, TempBuffer.GetData(), UncompressedSize, CompressedBuffers[OurBufferIndex].GetData(), CompressionBlock.GetCompressedSize());
					uint64 CopySize = FMath::Min<uint64>(UncompressedSize - OffsetInBlock, RemainingSize);
					check(UncompressedDestination + CopySize <= UncompressedBuffer.Data() + UncompressedBuffer.DataSize());
					FMemory::Memcpy(UncompressedDestination, TempBuffer.GetData() + OffsetInBlock, CopySize);
					UncompressedDestinationOffset += CopySize;
					RemainingSize -= CopySize;
				}
				else
				{
					check(UncompressedDestination + UncompressedSize <= UncompressedBuffer.Data() + UncompressedBuffer.DataSize());
					bUncompressed = FCompression::UncompressMemory(CompressionMethod, UncompressedDestination, UncompressedSize, CompressedBuffers[OurBufferIndex].GetData(), CompressionBlock.GetCompressedSize());
					UncompressedDestinationOffset += UncompressedSize;
					RemainingSize -= UncompressedSize;
				}
				if (!bUncompressed)
				{
					return FIoStatus(EIoErrorCode::ReadError, TEXT("Failed uncompressing chunk"));
				}
			}
			OffsetInBlock = 0;
		}
		return UncompressedBuffer;
	}

	TIoStatusOr<FIoStoreCompressedReadResult> ReadCompressed(const FIoChunkId& ChunkId, const FIoReadOptions& Options, bool bDecrypt) const
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ReadChunkCompressed);

		// Find where in the virtual file the chunk exists.
		const FIoOffsetAndLength* OffsetAndLength = TocReader.GetOffsetAndLength(ChunkId);
		if (!OffsetAndLength)
		{
			return FIoStatus(EIoErrorCode::UnknownChunkID, FString::Printf(TEXT("Unknown chunk ID '%s'"), *LexToString(ChunkId)));
		}

		// Combine with offset/size requested by the reader.
		uint64 RequestedOffset = Options.GetOffset();
		uint64 ResolvedOffset = OffsetAndLength->GetOffset() + RequestedOffset;
		uint64 ResolvedSize = 0;
		if (RequestedOffset <= OffsetAndLength->GetLength())
		{
			ResolvedSize = FMath::Min(Options.GetSize(), OffsetAndLength->GetLength() - RequestedOffset);
		}

		// Find what compressed blocks this read straddles.
		const FIoStoreTocResourceView& TocResource = TocReader.GetTocResource();
		const uint64 CompressionBlockSize = TocResource.Header.CompressionBlockSize;
		int32 FirstBlockIndex = int32(ResolvedOffset / CompressionBlockSize);
		int32 LastBlockIndex = int32((Align(ResolvedOffset + ResolvedSize, CompressionBlockSize) - 1) / CompressionBlockSize);

		// Determine size of the result and set up output buffers
		uint64 TotalCompressedSize = 0;
		uint64 TotalAlignedSize = 0;
		for (int32 BlockIndex = FirstBlockIndex; BlockIndex <= LastBlockIndex; ++BlockIndex)
		{
			const FIoStoreTocCompressedBlockEntry& CompressionBlock = TocResource.CompressionBlocks[BlockIndex];
			TotalCompressedSize += CompressionBlock.GetCompressedSize();
			TotalAlignedSize += Align(CompressionBlock.GetCompressedSize(), FAES::AESBlockSize);
		}

		FIoStoreCompressedReadResult Result;
		Result.IoBuffer = FIoBuffer(TotalAlignedSize);
		Result.Blocks.Reserve(LastBlockIndex + 1 - FirstBlockIndex);
		Result.UncompressedOffset = ResolvedOffset % CompressionBlockSize;
		Result.UncompressedSize = ResolvedSize;
		Result.TotalCompressedSize = TotalCompressedSize;

		// Set up the result blocks.
		uint64 CurrentOffset = 0;
		for (int32 BlockIndex = FirstBlockIndex; BlockIndex <= LastBlockIndex; ++BlockIndex)
		{
			const FIoStoreTocCompressedBlockEntry& CompressionBlock = TocResource.CompressionBlocks[BlockIndex];
			FIoStoreCompressedBlockInfo& BlockInfo = Result.Blocks.AddDefaulted_GetRef();
			
			BlockInfo.CompressionMethod = TocResource.CompressionMethods[CompressionBlock.GetCompressionMethodIndex()];
			BlockInfo.CompressedSize = CompressionBlock.GetCompressedSize();
			BlockInfo.UncompressedSize = CompressionBlock.GetUncompressedSize();
			BlockInfo.OffsetInBuffer = CurrentOffset;
			BlockInfo.AlignedSize = Align(CompressionBlock.GetCompressedSize(), FAES::AESBlockSize);
			CurrentOffset += BlockInfo.AlignedSize;
		}

		uint8* OutputBuffer = Result.IoBuffer.Data();

		// We can read the entire thing at once since we obligate the caller to skip the alignment padding.
		{
			const FIoStoreTocCompressedBlockEntry& CompressionBlock = TocResource.CompressionBlocks[FirstBlockIndex];
			int32 PartitionIndex = int32(CompressionBlock.GetOffset() / TocResource.Header.PartitionSize);
			int64 PartitionOffset = int64(CompressionBlock.GetOffset() % TocResource.Header.PartitionSize);

			std::atomic_bool bReadSucceeded;
			UE::Tasks::FTask ReadTask = StartAsyncRead(PartitionIndex, PartitionOffset, TotalAlignedSize, OutputBuffer, &bReadSucceeded);

			{
				TRACE_CPUPROFILER_EVENT_SCOPE(WaitForIo);
				ReadTask.Wait();
			}

			if (bReadSucceeded == false)
			{
				UE_LOG(LogIoStore, Error, TEXT("Read from container %s failed (partition %d, offset %" INT64_FMT ", size %" UINT64_FMT ")"), *ContainerPath, PartitionIndex, PartitionOffset, TotalAlignedSize);
				return FIoStoreCompressedReadResult();
			}
		}

		if (bDecrypt && EnumHasAnyFlags(TocResource.Header.ContainerFlags, EIoContainerFlags::Encrypted) )
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Decrypt);

			for (int32 BlockIndex = FirstBlockIndex; BlockIndex <= LastBlockIndex; ++BlockIndex)
			{
				FIoStoreCompressedBlockInfo& OutputBlock = Result.Blocks[BlockIndex - FirstBlockIndex];
				uint8* Buffer = OutputBuffer + OutputBlock.OffsetInBuffer;
				FAES::DecryptData(Buffer, OutputBlock.AlignedSize, TocReader.GetDecryptionKey());
			}
		}
		return Result;
	}

	const FIoDirectoryIndexReader& GetDirectoryIndexReader() const
	{
		return TocReader.GetDirectoryIndexReader();
	}

	bool TocChunkContainsBlockIndex(const int32 TocEntryIndex, const int32 BlockIndex) const
	{
		const FIoStoreTocResourceView& TocResource = TocReader.GetTocResource();
		const FIoOffsetAndLength& OffsetLength = TocResource.ChunkOffsetLengths[TocEntryIndex];

		const uint64 CompressionBlockSize = TocResource.Header.CompressionBlockSize;
		int32 FirstBlockIndex = int32(OffsetLength.GetOffset() / CompressionBlockSize);
		int32 LastBlockIndex = int32((Align(OffsetLength.GetOffset() + OffsetLength.GetLength(), CompressionBlockSize) - 1) / CompressionBlockSize);

		return BlockIndex >= FirstBlockIndex && BlockIndex <= LastBlockIndex;
	}

	uint64 GetPartitionSize() const
	{
		return TocReader.GetTocResource().Header.PartitionSize;
	}

	uint32 GetCompressionBlockSize() const
	{
		return TocReader.GetTocResource().Header.CompressionBlockSize;
	}
	
	const TArray<FName>& GetCompressionMethods() const
	{
		return TocReader.GetTocResource().CompressionMethods;
	}

	bool EnumerateCompressedBlocksForChunk(const FIoChunkId& ChunkId, TFunction<bool(const FIoStoreTocCompressedBlockInfo&)>&& Callback) const
	{
		const FIoOffsetAndLength* OffsetAndLength = TocReader.GetOffsetAndLength(ChunkId);
		if (!OffsetAndLength)
		{
			return false;
		}

		// Find what compressed blocks this chunk straddles.
		const FIoStoreTocResourceView& TocResource = TocReader.GetTocResource();
		const uint64 CompressionBlockSize = TocResource.Header.CompressionBlockSize;
		int32 FirstBlockIndex = int32(OffsetAndLength->GetOffset() / CompressionBlockSize);
		int32 LastBlockIndex = int32((Align(OffsetAndLength->GetOffset() + OffsetAndLength->GetLength(), CompressionBlockSize) - 1) / CompressionBlockSize);

		for (int32 BlockIndex = FirstBlockIndex; BlockIndex <= LastBlockIndex; ++BlockIndex)
		{
			const FIoStoreTocCompressedBlockEntry& Entry = TocResource.CompressionBlocks[BlockIndex];
			FIoStoreTocCompressedBlockInfo Info
			{
				Entry.GetOffset(),
				Entry.GetCompressedSize(),
				Entry.GetUncompressedSize(),
				Entry.GetCompressionMethodIndex()
			};
			if (!Callback(Info))
			{
				break;
			}
		}
		return true;
	}

	void EnumerateCompressedBlocks(TFunction<bool(const FIoStoreTocCompressedBlockInfo&)>&& Callback) const
	{
		const FIoStoreTocResourceView& TocResource = TocReader.GetTocResource();

		for (int32 BlockIndex = 0; BlockIndex < TocResource.CompressionBlocks.Num(); ++BlockIndex)
		{
			const FIoStoreTocCompressedBlockEntry& Entry = TocResource.CompressionBlocks[BlockIndex];
			FIoStoreTocCompressedBlockInfo Info
			{
				Entry.GetOffset(),
				Entry.GetCompressedSize(),
				Entry.GetUncompressedSize(),
				Entry.GetCompressionMethodIndex()
			};
			if (!Callback(Info))
			{
				break;
			}
		}
	}

	void GetContainerFilePaths(TArray<FString>& OutPaths)
	{
		TStringBuilder<256> Sb;

		for (uint32 PartitionIndex = 0; PartitionIndex < TocReader.GetTocResource().Header.PartitionCount; ++PartitionIndex)
		{
			Sb.Reset();
			Sb.Append(ContainerPath);
			if (PartitionIndex > 0)
			{
				Sb.Append(FString::Printf(TEXT("_s%d"), PartitionIndex));
			}
			Sb.Append(TEXT(".ucas"));
			OutPaths.Emplace(Sb);
		}
	}

private:


	FIoStoreTocReader TocReader;
	TArray<TUniquePtr<FContainerFileAccess>> ContainerFileAccessors;
	FString ContainerPath;
};

FIoStoreReader::FIoStoreReader()
	: Impl(new FIoStoreReaderImpl())
{
}

FIoStoreReader::~FIoStoreReader()
{
	delete Impl;
}

FIoStatus FIoStoreReader::Initialize(FStringView InContainerPath, const TMap<FGuid, FAES::FAESKey>& InDecryptionKeys)
{
	return Impl->Initialize(InContainerPath, InDecryptionKeys);
}

FIoContainerId FIoStoreReader::GetContainerId() const
{
	return Impl->GetContainerId();
}

uint32 FIoStoreReader::GetVersion() const
{
	return Impl->GetVersion();
}

EIoContainerFlags FIoStoreReader::GetContainerFlags() const
{
	return Impl->GetContainerFlags();
}

FGuid FIoStoreReader::GetEncryptionKeyGuid() const
{
	return Impl->GetEncryptionKeyGuid();
}

int32 FIoStoreReader::GetChunkCount() const
{
	return Impl->GetChunkCount();
}

FString FIoStoreReader::GetContainerName() const
{
	return Impl->GetContainerName();
}

void FIoStoreReader::EnumerateChunks(TFunction<bool(FIoStoreTocChunkInfo&&)>&& Callback) const
{
	Impl->EnumerateChunks(MoveTemp(Callback));
}

TIoStatusOr<FIoStoreTocChunkInfo> FIoStoreReader::GetChunkInfo(const FIoChunkId& Chunk) const
{
	return Impl->GetChunkInfo(Chunk);
}

TIoStatusOr<FIoStoreTocChunkInfo> FIoStoreReader::GetChunkInfo(const uint32 TocEntryIndex) const
{
	return Impl->GetChunkInfo(TocEntryIndex);
}

TIoStatusOr<FIoBuffer> FIoStoreReader::Read(const FIoChunkId& Chunk, const FIoReadOptions& Options) const
{
	return Impl->Read(Chunk, Options);
}

TIoStatusOr<FIoStoreCompressedReadResult> FIoStoreReader::ReadCompressed(const FIoChunkId& Chunk, const FIoReadOptions& Options, bool bDecrypt) const
{
	return Impl->ReadCompressed(Chunk, Options, bDecrypt);
}

UE::Tasks::TTask<TIoStatusOr<FIoBuffer>> FIoStoreReader::ReadAsync(const FIoChunkId& Chunk, const FIoReadOptions& Options) const
{
	return Impl->ReadAsync(Chunk, Options);
}

const FIoDirectoryIndexReader& FIoStoreReader::GetDirectoryIndexReader() const
{
	return Impl->GetDirectoryIndexReader();
}

uint64 FIoStoreReader::GetPartitionSize() const
{
	return Impl->GetPartitionSize();
}

uint32 FIoStoreReader::GetCompressionBlockSize() const
{
	return Impl->GetCompressionBlockSize();
}

const TArray<FName>& FIoStoreReader::GetCompressionMethods() const
{
	return Impl->GetCompressionMethods();
}

void FIoStoreReader::EnumerateCompressedBlocks(TFunction<bool(const FIoStoreTocCompressedBlockInfo&)>&& Callback) const
{
	Impl->EnumerateCompressedBlocks(MoveTemp(Callback));
}

void FIoStoreReader::EnumerateCompressedBlocksForChunk(const FIoChunkId& Chunk, TFunction<bool(const FIoStoreTocCompressedBlockInfo&)>&& Callback) const
{
	Impl->EnumerateCompressedBlocksForChunk(Chunk, MoveTemp(Callback));
}

void FIoStoreReader::GetContainerFilePaths(TArray<FString>& OutPaths)
{
	Impl->GetContainerFilePaths(OutPaths);
}

static TAutoConsoleVariable<bool> CVarIoStoreAllowMemoryMappedUtoc(
	TEXT("IoStore.AllowMemoryMappedUtoc"),
	true,
	TEXT("Allow IoStore to memory map utoc containers instead of loading thier content as a whole into memory.")
	);

FIoStoreTocResourceStorage::FIoStoreTocResourceStorage(const TCHAR* TocFilePath)
{
	IPlatformFile& Ipf = FPlatformFileManager::Get().GetPlatformFile();

	// try memory mapping first
	if (FPlatformProperties::SupportsMemoryMappedFiles() && CVarIoStoreAllowMemoryMappedUtoc.GetValueOnAnyThread())
	{
		FOpenMappedResult Result = Ipf.OpenMappedEx(TocFilePath, IPlatformFile::EOpenReadFlags::None);
		if (Result.HasValue())
		{
			FMappedFile Mapped;
			Mapped.MappedFile = Result.StealValue();
			Data.Emplace<FMappedFile>(MoveTemp(Mapped));
		}
	}
	// then try to open file directly and own all read blocks.
	if (Data.IsType<FEmptyVariantState>())
	{
		FFileOpenResult Result = Ipf.OpenRead(TocFilePath, IPlatformFile::EOpenReadFlags::None);
		if (Result.HasValue())
		{
			FReadBlocks Blocks;
			Blocks.File = Result.StealValue();
			Data.Emplace<FReadBlocks>(MoveTemp(Blocks));
		}
	}
}

uint64 FIoStoreTocResourceStorage::GetAllocatedSize() const
{
	auto Visitor = [](auto&& V) -> int64
	{
		using ValueType = std::decay_t<decltype(V)>;
		if constexpr (std::is_same_v<ValueType, FReadBlocks>)
		{
			return V.Blocks.GetAllocatedSize();
		}
		else if constexpr (std::is_same_v<ValueType, FMappedFile>)
		{
			return sizeof(*V.MappedFile) + V.MappedRegions.GetAllocatedSize() + V.OwnedRegions.GetAllocatedSize();
		}
		else
		{
			return 0ll;
		}
	};
	return sizeof(*this) + Visit(Visitor, Data);
}

TConstArrayView<uint8> FIoStoreTocResourceStorage::ChopBytes(FReadBlocks& Blocks, int32 Size)
{
	if (Size <= 0)
	{
		return TConstArrayView<uint8>();
	}
	auto& Range = Blocks.Blocks.AddDefaulted_GetRef();
	Range.SetNumUninitialized(Size);
	if (Blocks.File->Read(Range.GetData(), Size))
	{
		return MakeConstArrayView(Range);
	}
	Blocks.Blocks.Pop();
	Blocks.File->SeekFromEnd();
	return TConstArrayView<uint8>();
}

TConstArrayView<uint8> FIoStoreTocResourceStorage::ChopBytes(FMappedFile& MappedFile, int32 Size)
{
	if (Size <= 0)
	{
		return TConstArrayView<uint8>();
	}
	TUniquePtr<IMappedFileRegion> Region(MappedFile.MappedFile->MapRegion(MappedFile.Cursor, Size));
	const int32 ActualSize = static_cast<int32>(Region->GetMappedSize());
	ensureAlways(ActualSize == Size);
	// optimization for small regions
	if (ActualSize < FPlatformMemory::GetConstants().OsAllocationGranularity)
	{
		auto& Range = MappedFile.OwnedRegions.Emplace_GetRef(Region->GetMappedPtr(), ActualSize);
		Region.Reset(); // intentionally drop mapped Region to let the OS to unload the page
		MappedFile.Cursor += ActualSize;
		MappedFile.bLastReadBlockWasOwned = true;
		return MakeConstArrayView(Range);
	}
	else
	{
		auto& Range = MappedFile.MappedRegions.Add_GetRef(MoveTemp(Region));
		MappedFile.Cursor += ActualSize;
		MappedFile.bLastReadBlockWasOwned = false;
		return TConstArrayView<uint8>(Range->GetMappedPtr(), ActualSize);
	}
}

void FIoStoreTocResourceStorage::ReleaseOwnershipOfLastBlock()
{
	auto Visitor = [](auto&& V)
	{
		using ValueType = std::decay_t<decltype(V)>;
		auto NoShrink = EAllowShrinking::No;
		if constexpr (std::is_same_v<ValueType, FReadBlocks>)
		{
			V.Blocks.Pop(NoShrink);
		}
		else if constexpr (std::is_same_v<ValueType, FMappedFile>)
		{
			if (V.bLastReadBlockWasOwned)
			{
				V.OwnedRegions.Pop(NoShrink);
			}
			else
			{
				V.MappedRegions.Pop(NoShrink);
			}
		}
	};
	Visit(Visitor, Data);
}

void FIoStoreTocResourceStorage::FinalizeRead()
{
	auto Visitor = [](auto&& V)
	{
		using ValueType = std::decay_t<decltype(V)>;
		if constexpr (std::is_same_v<ValueType, FReadBlocks>)
		{
			V.File.Reset();
		}
	};
	Visit(Visitor, Data);
}

FIoStatus FIoStoreTocResourceView::Read(const TCHAR* TocFilePath, EIoStoreTocReadOptions ReadOptions, FIoStoreTocResourceView& OutTocResource, FIoStoreTocResourceStorage& OutTocStorage)
{
	check(TocFilePath != nullptr);

	FIoStoreTocResourceStorage TocStorage(TocFilePath);

	if (!TocStorage.IsLoaded())
	{
		return FIoStatusBuilder(EIoErrorCode::FileOpenFailed) << TEXT("Failed to open IoStore TOC file '") << TocFilePath << TEXT("'");
	}

	// Header
	FIoStoreTocHeader& Header = OutTocResource.Header;
	if (TArray<FIoStoreTocHeader> HeaderView = TocStorage.ChopArray<FIoStoreTocHeader>(1); !HeaderView.IsEmpty())
	{
		FPlatformMemory::Memcpy(&Header, HeaderView.GetData(), HeaderView.NumBytes());
	}
	else
	{
		return FIoStatusBuilder(EIoErrorCode::CorruptToc) << TEXT("Failed to read IoStore TOC file '") << TocFilePath << TEXT("'");
	}

	if (!Header.CheckMagic())
	{
		return FIoStatusBuilder(EIoErrorCode::CorruptToc) << TEXT("TOC header magic mismatch while reading '") << TocFilePath << TEXT("'");
	}

	if (Header.TocHeaderSize != sizeof(FIoStoreTocHeader))
	{
		return FIoStatusBuilder(EIoErrorCode::CorruptToc) << TEXT("TOC header size mismatch while reading '") << TocFilePath << TEXT("'");
	}

	if (Header.TocCompressedBlockEntrySize != sizeof(FIoStoreTocCompressedBlockEntry))
	{
		return FIoStatusBuilder(EIoErrorCode::CorruptToc) << TEXT("TOC compressed block entry size mismatch while reading '") << TocFilePath << TEXT("'");
	}

	if (Header.Version < static_cast<uint8>(EIoStoreTocVersion::DirectoryIndex))
	{
		return FIoStatusBuilder(EIoErrorCode::CorruptToc) << TEXT("Outdated TOC header version while reading '") << TocFilePath << TEXT("'");
	}

	if (Header.Version > static_cast<uint8>(EIoStoreTocVersion::Latest))
	{
		return FIoStatusBuilder(EIoErrorCode::CorruptToc) << TEXT("Too new TOC header version while reading '") << TocFilePath << TEXT("'");
	}

	// Chunk IDs
	OutTocResource.ChunkIds = TocStorage.ChopView<FIoChunkId>(Header.TocEntryCount);

	// Chunk offsets
	OutTocResource.ChunkOffsetLengths = TocStorage.ChopView<FIoOffsetAndLength>(Header.TocEntryCount);

	// Chunk perfect hash map
	uint32 PerfectHashSeedsCount = 0;
	uint32 ChunksWithoutPerfectHashCount = 0;
	if (Header.Version >= static_cast<uint8>(EIoStoreTocVersion::PerfectHashWithOverflow))
	{
		PerfectHashSeedsCount = Header.TocChunkPerfectHashSeedsCount;
		ChunksWithoutPerfectHashCount = Header.TocChunksWithoutPerfectHashCount;
	}
	else if (Header.Version >= static_cast<uint8>(EIoStoreTocVersion::PerfectHash))
	{
		PerfectHashSeedsCount = Header.TocChunkPerfectHashSeedsCount;
	}
	if (PerfectHashSeedsCount)
	{
		OutTocResource.ChunkPerfectHashSeeds = TocStorage.ChopView<int32>(PerfectHashSeedsCount);
	}
	if (ChunksWithoutPerfectHashCount)
	{
		OutTocResource.ChunkIndicesWithoutPerfectHash = TocStorage.ChopView<int32>(ChunksWithoutPerfectHashCount);
	}

	// Compression blocks
	OutTocResource.CompressionBlocks = TocStorage.ChopView<FIoStoreTocCompressedBlockEntry>(Header.TocCompressedBlockEntryCount);

	// Compression methods
	{
		OutTocResource.CompressionMethods.Reserve(Header.CompressionMethodNameCount + 1);
		OutTocResource.CompressionMethods.Add(NAME_None);
		TArray<ANSICHAR> AnsiCompressionMethodNamesBlock = TocStorage.ChopArray<ANSICHAR>(Header.CompressionMethodNameCount * Header.CompressionMethodNameLength);
		const ANSICHAR* AnsiCompressionMethodNames = reinterpret_cast<const ANSICHAR*>(AnsiCompressionMethodNamesBlock.GetData());
		for (uint32 CompressonNameIndex = 0; CompressonNameIndex < Header.CompressionMethodNameCount; CompressonNameIndex++)
		{
			const ANSICHAR* AnsiCompressionMethodName = AnsiCompressionMethodNames + CompressonNameIndex * Header.CompressionMethodNameLength;
			OutTocResource.CompressionMethods.Add(FName(AnsiCompressionMethodName));
		}
	}

	// Chunk block signatures
	const bool bIsSigned = EnumHasAnyFlags(Header.ContainerFlags, EIoContainerFlags::Signed);
	if (IsSigningEnabled() || bIsSigned)
	{
		if (!bIsSigned)
		{
			return FIoStatus(EIoErrorCode::SignatureError, TEXT("Missing signature"));
		}

		const int32 HashSize = TocStorage.ChopArray<int32>(1)[0];

		TArray<uint8> BothSignatures;
		BothSignatures.Reserve(2 * HashSize);
		BothSignatures.Append(TocStorage.ChopArray<uint8>(HashSize));
		BothSignatures.Append(TocStorage.ChopArray<uint8>(HashSize));
		FSHA1::HashBuffer(BothSignatures.GetData(), BothSignatures.Num(), OutTocResource.SignatureHash.Hash);

		TArrayView<uint8> TocSignature = MakeArrayView(BothSignatures.GetData(), HashSize);
		TArrayView<uint8> BlockSignature = MakeArrayView(BothSignatures.GetData() + HashSize, HashSize);

		OutTocResource.ChunkBlockSignatures = TocStorage.ChopView<FSHAHash>(Header.TocCompressedBlockEntryCount);

		if (IsSigningEnabled())
		{
			FIoStatus SignatureStatus = ValidateContainerSignature(
				GetPublicSigningKey(),
				Header,
				OutTocResource.ChunkBlockSignatures,
				TocSignature,
				BlockSignature);
			if (!SignatureStatus.IsOk())
			{
				return SignatureStatus;
			}
		}
	}

	// Directory index
	if (EnumHasAnyFlags(ReadOptions, EIoStoreTocReadOptions::ReadDirectoryIndex) &&
		EnumHasAnyFlags(Header.ContainerFlags, EIoContainerFlags::Indexed) &&
		Header.DirectoryIndexSize > 0)
	{
		OutTocResource.DirectoryIndexBuffer = TocStorage.ChopView<uint8>(Header.DirectoryIndexSize);
	}

	// Meta
	if (EnumHasAnyFlags(ReadOptions, EIoStoreTocReadOptions::ReadTocMeta))
	{
		if (Header.Version >= static_cast<uint8>(EIoStoreTocVersion::ReplaceIoChunkHashWithIoHash))
		{
			OutTocResource.ChunkMetas = TocStorage.ChopView<FIoStoreTocEntryMeta>(Header.TocEntryCount);
		}
		else
		{
			struct FIoStoreTocEntryMetaOld
			{
				uint8 ChunkHash[32];
				FIoStoreTocEntryMetaFlags Flags;
			};
			TArray<FIoStoreTocEntryMetaOld> OldChunkMetas = TocStorage.ChopArray<FIoStoreTocEntryMetaOld>(Header.TocEntryCount);
			OutTocResource.LegacyChunkMetas.Reserve(OldChunkMetas.Num());
			for (const FIoStoreTocEntryMetaOld& OldChunkMeta : OldChunkMetas)
			{
				FIoStoreTocEntryMeta& ChunkMeta = OutTocResource.LegacyChunkMetas.Emplace_GetRef();
				FMemory::Memcpy(ChunkMeta.ChunkHash.GetBytes(), &OldChunkMeta.ChunkHash, sizeof ChunkMeta.ChunkHash);
				ChunkMeta.Flags = OldChunkMeta.Flags;
			}
			OutTocResource.ChunkMetas = MakeConstArrayView(OutTocResource.LegacyChunkMetas);
		}
	}

	if (Header.Version < static_cast<uint8>(EIoStoreTocVersion::PartitionSize))
	{
		Header.PartitionCount = 1;
		Header.PartitionSize = MAX_uint64;
	}

	TocStorage.FinalizeRead();
	OutTocStorage = MoveTemp(TocStorage);

	return FIoStatus::Ok;
}

FIoStoreTocChunkInfo FIoStoreTocResource::GetTocChunkInfo(int32 TocEntryIndex) const
{
	return GetTocChunkInfoInternal(*this, TocEntryIndex);
}

FIoStoreTocResource FIoStoreTocResource::BuildResourceFromMappedView(const FIoStoreTocResourceView& View)
{
	FIoStoreTocResource Resource;
	Resource.Header = View.Header;
	Resource.ChunkIds = View.ChunkIds;
	Resource.ChunkOffsetLengths = View.ChunkOffsetLengths;
	Resource.ChunkPerfectHashSeeds = View.ChunkPerfectHashSeeds;
	Resource.ChunkIndicesWithoutPerfectHash = View.ChunkIndicesWithoutPerfectHash;
	Resource.CompressionBlocks = View.CompressionBlocks;

	Resource.CompressionMethods = View.CompressionMethods;
	Resource.SignatureHash = View.SignatureHash;
	Resource.ChunkBlockSignatures = View.ChunkBlockSignatures;
	Resource.DirectoryIndexBuffer = View.DirectoryIndexBuffer;
	Resource.ChunkMetas = View.ChunkMetas;

	return Resource;
}

FIoStatus FIoStoreTocResource::Read(const TCHAR* TocFilePath, EIoStoreTocReadOptions ReadOptions, FIoStoreTocResource& OutTocResource)
{
	FIoStoreTocResourceView View;
	FIoStoreTocResourceStorage Storage;
	FIoStatus Status = FIoStoreTocResourceView::Read(TocFilePath, ReadOptions, View, Storage);

	if (Status.IsOk())
	{
		OutTocResource = BuildResourceFromMappedView(View);
	}

	return Status;
}

static FString FormatLastErrorMessage(const TCHAR* PrefixMessage)
{
	TCHAR Buffer[256];
	uint32 LastError = FPlatformMisc::GetLastError();
	return FString::Printf(TEXT("%s Error: (%u: %s)."), PrefixMessage, LastError,
		FPlatformMisc::GetSystemErrorMessage(Buffer, UE_ARRAY_COUNT(Buffer), LastError));
}

TIoStatusOr<uint64> FIoStoreTocResource::Write(
	const TCHAR* TocFilePath,
	FIoStoreTocResource& TocResource,
	uint32 CompressionBlockSize,
	uint64 MaxPartitionSize,
	const FIoContainerSettings& ContainerSettings)
{
	check(TocFilePath != nullptr);

	IPlatformFile& Ipf = FPlatformFileManager::Get().GetPlatformFile();
	TUniquePtr<IFileHandle> TocFileHandle(Ipf.OpenWrite(TocFilePath, /* append */ false, /* allowread */ true));

	if (!TocFileHandle)
	{
		FIoStatus Status = FIoStatusBuilder(EIoErrorCode::FileOpenFailed) << TEXT("Failed to open IoStore TOC file '") << TocFilePath << TEXT("'");
		return Status;
	}

	if (TocResource.ChunkIds.Num() != TocResource.ChunkOffsetLengths.Num())
	{
		return FIoStatus(EIoErrorCode::InvalidParameter, TEXT("Number of TOC chunk IDs doesn't match the number of offsets"));
	}

	if (TocResource.ChunkIds.Num() != TocResource.ChunkMetas.Num())
	{
		return FIoStatus(EIoErrorCode::InvalidParameter, TEXT("Number of TOC chunk IDs doesn't match the number of chunk meta data"));
	}

	bool bHasExplicitCompressionMethodNone = false;
	for (int32 CompressionMethodIndex = 0; CompressionMethodIndex < TocResource.CompressionMethods.Num(); ++CompressionMethodIndex)
	{
		if (TocResource.CompressionMethods[CompressionMethodIndex].IsNone())
		{
			if (CompressionMethodIndex != 0)
			{
				return FIoStatus(EIoErrorCode::InvalidParameter, TEXT("Compression method None must be the first compression method"));
			}
			bHasExplicitCompressionMethodNone = true;
		}
	}

	FMemory::Memzero(&TocResource.Header, sizeof(FIoStoreTocHeader));

	FIoStoreTocHeader& TocHeader = TocResource.Header;
	TocHeader.MakeMagic();
	TocHeader.Version = static_cast<uint8>(EIoStoreTocVersion::Latest);
	TocHeader.TocHeaderSize = sizeof(TocHeader);
	TocHeader.TocEntryCount = TocResource.ChunkIds.Num();
	TocHeader.TocChunkPerfectHashSeedsCount = TocResource.ChunkPerfectHashSeeds.Num();
	TocHeader.TocChunksWithoutPerfectHashCount = TocResource.ChunkIndicesWithoutPerfectHash.Num();
	TocHeader.TocCompressedBlockEntryCount = TocResource.CompressionBlocks.Num();
	TocHeader.TocCompressedBlockEntrySize = sizeof(FIoStoreTocCompressedBlockEntry);
	TocHeader.CompressionBlockSize = CompressionBlockSize;
	TocHeader.CompressionMethodNameCount = TocResource.CompressionMethods.Num() - (bHasExplicitCompressionMethodNone ? 1 : 0);
	TocHeader.CompressionMethodNameLength = FIoStoreTocResource::CompressionMethodNameLen;
	TocHeader.DirectoryIndexSize = TocResource.DirectoryIndexBuffer.Num();
	TocHeader.ContainerId = ContainerSettings.ContainerId;
	TocHeader.EncryptionKeyGuid = ContainerSettings.EncryptionKeyGuid;
	TocHeader.ContainerFlags = ContainerSettings.ContainerFlags;
	if (TocHeader.TocEntryCount == 0)
	{
		TocHeader.PartitionCount = 0;
		TocHeader.PartitionSize = MAX_uint64;
	}
	else if (MaxPartitionSize)
	{
		const FIoStoreTocCompressedBlockEntry& LastBlock = TocResource.CompressionBlocks.Last();
		uint64 LastBlockEnd = LastBlock.GetOffset() + LastBlock.GetCompressedSize() - 1;
		TocHeader.PartitionCount = IntCastChecked<uint32>(LastBlockEnd / MaxPartitionSize + 1);
		check(TocHeader.PartitionCount > 0);
		TocHeader.PartitionSize = MaxPartitionSize;
	}
	else
	{
		TocHeader.PartitionCount = 1;
		TocHeader.PartitionSize = MAX_uint64;
	}

	TocFileHandle->Seek(0);

	// Header
	if (!TocFileHandle->Write(reinterpret_cast<const uint8*>(&TocResource.Header), sizeof(FIoStoreTocHeader)))
	{
		return FIoStatus(EIoErrorCode::WriteError, FormatLastErrorMessage(TEXT("Failed to write TOC header.")));
	}

	// Chunk IDs
	if (!WriteArray(TocFileHandle.Get(), TocResource.ChunkIds))
	{
		return FIoStatus(EIoErrorCode::WriteError, FormatLastErrorMessage(TEXT("Failed to write chunk ids.")));
	}

	// Chunk offsets
	if (!WriteArray(TocFileHandle.Get(), TocResource.ChunkOffsetLengths))
	{
		return FIoStatus(EIoErrorCode::WriteError, FormatLastErrorMessage(TEXT("Failed to write chunk offsets.")));
	}

	// Chunk perfect hash map
	if (!WriteArray(TocFileHandle.Get(), TocResource.ChunkPerfectHashSeeds))
	{
		return FIoStatus(EIoErrorCode::WriteError, FormatLastErrorMessage(TEXT("Failed to write chunk hash seeds.")));
	}
	if (!WriteArray(TocFileHandle.Get(), TocResource.ChunkIndicesWithoutPerfectHash))
	{
		return FIoStatus(EIoErrorCode::WriteError, FormatLastErrorMessage(TEXT("Failed to write chunk indices without perfect hash.")));
	}

	// Compression blocks
	if (!WriteArray(TocFileHandle.Get(), TocResource.CompressionBlocks))
	{
		return FIoStatus(EIoErrorCode::WriteError, FormatLastErrorMessage(TEXT("Failed to write chunk block entries.")));
	}

	// Compression methods
	ANSICHAR AnsiMethodName[FIoStoreTocResource::CompressionMethodNameLen];

	for (FName MethodName : TocResource.CompressionMethods)
	{
		if (MethodName.IsNone())
		{
			continue;
		}
		FMemory::Memzero(AnsiMethodName, FIoStoreTocResource::CompressionMethodNameLen);
		FCStringAnsi::Strncpy(AnsiMethodName, TCHAR_TO_ANSI(*MethodName.ToString()), FIoStoreTocResource::CompressionMethodNameLen);

		if (!TocFileHandle->Write(reinterpret_cast<const uint8*>(AnsiMethodName), FIoStoreTocResource::CompressionMethodNameLen))
		{
			return FIoStatus(EIoErrorCode::WriteError, FormatLastErrorMessage(TEXT("Failed to write compression method TOC entry.")));
		}
	}

	// Chunk block signatures
	if (EnumHasAnyFlags(TocHeader.ContainerFlags, EIoContainerFlags::Signed))
	{
		TArray<uint8> TocSignature, BlockSignature;
		check(TocResource.ChunkBlockSignatures.Num() == TocResource.CompressionBlocks.Num());

		FIoStatus SignatureStatus = CreateContainerSignature(
			ContainerSettings.SigningKey,
			TocHeader,
			TocResource.ChunkBlockSignatures,
			TocSignature,
			BlockSignature);

		if (!SignatureStatus .IsOk())
		{
			return SignatureStatus;
		}

		check(TocSignature.Num() == BlockSignature.Num());

		const int32 HashSize = TocSignature.Num();
		TocFileHandle->Write(reinterpret_cast<const uint8*>(&HashSize), sizeof(int32));
		TocFileHandle->Write(TocSignature.GetData(), HashSize);
		TocFileHandle->Write(BlockSignature.GetData(), HashSize);

		if (!WriteArray(TocFileHandle.Get(), TocResource.ChunkBlockSignatures))
		{
			return FIoStatus(EIoErrorCode::WriteError, FormatLastErrorMessage(TEXT("Failed to write chunk block signatures.")));
		}
	}

	// Directory index (EIoStoreTocReadOptions::ReadDirectoryIndex)
	if (EnumHasAnyFlags(TocHeader.ContainerFlags, EIoContainerFlags::Indexed))
	{
		if (!TocFileHandle->Write(TocResource.DirectoryIndexBuffer.GetData(), TocResource.DirectoryIndexBuffer.Num()))
		{
			return FIoStatus(EIoErrorCode::WriteError, FormatLastErrorMessage(TEXT("Failed to write directory index buffer.")));
		}
	}

	// Meta data (EIoStoreTocReadOptions::ReadTocMeta)
	if (!WriteArray(TocFileHandle.Get(), TocResource.ChunkMetas))
	{
		return FIoStatus(EIoErrorCode::WriteError, FormatLastErrorMessage(TEXT("Failed to write chunk meta data.")));
	}

	TocFileHandle->Flush(true);

	return TocFileHandle->Tell();
}

FIoStoreTocResourceView FIoStoreTocResource::AsView() const
{
	FIoStoreTocResourceView Result = {};
	Result.Header = Header;
	Result.ChunkIds = ChunkIds;
	Result.ChunkOffsetLengths = ChunkOffsetLengths;
	Result.ChunkPerfectHashSeeds = ChunkPerfectHashSeeds;
	Result.ChunkIndicesWithoutPerfectHash = ChunkIndicesWithoutPerfectHash;
	Result.CompressionBlocks = CompressionBlocks;
	Result.CompressionMethods = CompressionMethods;
	Result.SignatureHash = SignatureHash;
	Result.ChunkBlockSignatures = ChunkBlockSignatures;
	Result.DirectoryIndexBuffer = DirectoryIndexBuffer;
	Result.ChunkMetas = ChunkMetas;
	Result.LegacyChunkMetas = {};
	return Result;
}

uint64 FIoStoreTocResource::HashChunkIdWithSeed(int32 Seed, const FIoChunkId& ChunkId)
{
	const uint8* Data = ChunkId.GetData();
	const uint32 DataSize = ChunkId.GetSize();
	uint64 Hash = Seed ? static_cast<uint64>(Seed) : 0xcbf29ce484222325;
	for (uint32 Index = 0; Index < DataSize; ++Index)
	{
		Hash = (Hash * 0x00000100000001B3) ^ Data[Index];
	}
	return Hash;
}

void FIoStoreReader::GetFilenames(TArray<FString>& OutFileList) const
{
	const FIoDirectoryIndexReader& DirectoryIndex = GetDirectoryIndexReader();

	DirectoryIndex.IterateDirectoryIndex(
		FIoDirectoryIndexHandle::RootDirectory(),
		TEXT(""),
		[&OutFileList](FStringView Filename, uint32 TocEntryIndex) -> bool
		{
			OutFileList.AddUnique(FString(Filename));
			return true;
		});
}

void FIoStoreReader::GetFilenamesByBlockIndex(const TArray<int32>& InBlockIndexList, TArray<FString>& OutFileList) const
{
	const FIoDirectoryIndexReader& DirectoryIndex = GetDirectoryIndexReader();

	DirectoryIndex.IterateDirectoryIndex(FIoDirectoryIndexHandle::RootDirectory(), TEXT(""),
		[this, &InBlockIndexList, &OutFileList](FStringView Filename, uint32 TocEntryIndex) -> bool
		{
			for (int32 BlockIndex : InBlockIndexList)
			{
				if (Impl->TocChunkContainsBlockIndex(TocEntryIndex, BlockIndex))
				{
					OutFileList.AddUnique(FString(Filename));
					break;
				}
			}

			return true;
		});
}
