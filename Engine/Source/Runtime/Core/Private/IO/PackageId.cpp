// Copyright Epic Games, Inc. All Rights Reserved.

#include "IO/PackageId.h"

#include "HAL/LowLevelMemTracker.h"
#include "Hash/CityHash.h"
#include "Misc/Char.h"
#include "Misc/Parse.h"
#include "Misc/ScopeRWLock.h"
#include "Misc/TransactionallySafeRWLock.h"
#include "Serialization/CompactBinaryWriter.h"
#include "Serialization/StructuredArchiveAdapters.h"
#include "Serialization/StructuredArchiveSlots.h"

#if WITH_PACKAGEID_NAME_MAP
LLM_DEFINE_TAG(PackageId_ReverseMapping);

namespace PackageIdImpl
{
	// We use a transactionally-safe lock here because `FPackageId::FromName` can be called by 
	// `FPackageName::DoesPackageExist`, and this should be safe to check inside of a transaction.
	FTransactionallySafeRWLock Lock;
	TMap<uint64, FName> Entries;
}
#endif

FPackageId FPackageId::FromName(const FName& Name)
{
	TCHAR NameStr[FName::StringBufferSize + 2];
	uint32 NameLen = Name.ToString(NameStr);

	for (uint32 I = 0; I < NameLen; ++I)
	{
		NameStr[I] = TChar<TCHAR>::ToLower(NameStr[I]);
	}
	uint64 Hash = CityHash64(reinterpret_cast<const char*>(NameStr), NameLen * sizeof(TCHAR));
	checkf(Hash != InvalidId, TEXT("Package name hash collision \"%s\" and InvalidId"), NameStr);

#if WITH_PACKAGEID_NAME_MAP
	{
		UE::TWriteScopeLock ScopeWriteLock(PackageIdImpl::Lock);
		LLM_SCOPE_BYTAG(PackageId_ReverseMapping);
		FName EntryName = PackageIdImpl::Entries.FindOrAdd(Hash, Name);
		checkf(EntryName.GetDisplayIndex() == Name.GetDisplayIndex() || EntryName.GetComparisonIndex() == Name.GetComparisonIndex(), TEXT("FPackageId collision: %llu for both %s and %s"), Hash, *Name.ToString(), *EntryName.ToString());
	}
#endif
	return FPackageId(Hash);
}

#if WITH_PACKAGEID_NAME_MAP
FName FPackageId::GetName() const
{
	UE::TReadScopeLock ScopeReadLock(PackageIdImpl::Lock);
	return PackageIdImpl::Entries.FindRef(Id);
}
#endif

FArchive& operator<<(FArchive& Ar, FPackageId& Value)
{
	FStructuredArchiveFromArchive(Ar).GetSlot() << Value;
	return Ar;
}

void operator<<(FStructuredArchiveSlot Slot, FPackageId& Value)
{
	Slot << Value.Id;
}

void SerializeForLog(FCbWriter& Writer, const FPackageId& Value)
{
	Writer.BeginObject();
	Writer.AddString(ANSITEXTVIEW("$type"), ANSITEXTVIEW("PackageId"));

	TUtf8StringBuilder<FName::StringBufferSize> TextBuilder;
	TextBuilder.Appendf("0x%llX", Value.Id);

#if WITH_PACKAGEID_NAME_MAP
	const FName Name = Value.GetName();
	TextBuilder << " (" << Name << ")";
#endif // WITH_PACKAGEID_NAME_MAP

	Writer.AddString(ANSITEXTVIEW("$text"), TextBuilder);
	Writer.AddInteger(ANSITEXTVIEW("Id"), Value.Id);

#if WITH_PACKAGEID_NAME_MAP
	Writer.AddString(ANSITEXTVIEW("Name"), WriteToUtf8String<FName::StringBufferSize>(Name));
#endif // WITH_PACKAGEID_NAME_MAP

	Writer.EndObject();
}

FString LexToString(const FPackageId& PackageId)
{
	return FString::Printf(TEXT("%llX"), PackageId.Value());
}

void LexFromString(FPackageId& Out, FStringView String)
{
	if (String.IsEmpty())
	{
		Out = FPackageId();
		return;
	}

	if (String.StartsWith(TEXT("0x"), ESearchCase::IgnoreCase))
	{
		String.RightChopInline(2);
	}

	if (String.Len() == sizeof(uint64) * 2)
	{
		Out = FPackageId::FromValue(FParse::HexNumber64(String));
	}
	else
	{
		// Assume string is a long package name
		Out = FPackageId::FromName(FName(String));
	}
}

void LexFromString(FPackageId& Out, const TCHAR* String)
{
	if (String != nullptr)
	{
		LexFromString(Out, FStringView(String));
	}
	else
	{
		Out = FPackageId();
	}
}

