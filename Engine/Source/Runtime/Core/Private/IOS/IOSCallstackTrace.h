// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ProfilingDebugging/CallstackTrace.h"

#if UE_CALLSTACK_TRACE_ENABLED

#include "CoreTypes.h"

uint32 CallstackTrace_GetExternalCallstackId(uint64* Frames, uint32 Count);

#endif
