// Copyright Epic Games, Inc. All Rights Reserved.

#include "IOSChunkInstaller.h"

#include "Apple/AppleStringUtils.h"
#include "Misc/CallbackDevice.h"

FIOSChunkInstall::FIOSChunkInstall()
{
    // TODO: call conditionallyBeginAccess on all of the possible chunks
    GetChunkLocation(1);
}


FIOSChunkInstall::~FIOSChunkInstall()
{
    ShutDown();
}


void FIOSChunkInstall::ShutDown()
{
    // call endAccess on all requests
    for ( TMap<uint32, FIOSChunkStatus>::TIterator It(ChunkStatus); It; ++It )
    {
        FIOSChunkStatus Status = It.Value();
        [Status.Request endAccessingResources];
    }
}


EChunkLocation::Type FIOSChunkInstall::GetChunkLocation( uint32 ChunkID )
{
    return GetChunkLocation(ChunkID, false);
}


EChunkLocation::Type FIOSChunkInstall::GetChunkLocation( uint32 ChunkID, bool bSkipMount )
{
    // check for cached location
    FIOSChunkStatus* CurStatus = ChunkStatus.Find(ChunkID);
    
    // early out if we already know it's local (should be the most common case)
    if (CurStatus && (CurStatus->Request.progress.fractionCompleted == 1.0f) && MountedChunks.Find(ChunkID))
    {
        return EChunkLocation::LocalFast;
    }
    
    // update the cached chunk status
    if (!CurStatus)
    {
        PrioritizeChunk(ChunkID, EChunkPriority::Immediate);
    }
    
    return EChunkLocation::NotAvailable;
}


bool FIOSChunkInstall::GetProgressReportingTypeSupported( EChunkProgressReportingType::Type ReportType )
{
    bool bSupported = false;
    switch (ReportType)
    {
        case EChunkProgressReportingType::PercentageComplete:
            bSupported = true;
            break;
        default:
            bSupported = false;
            break;
    }
    return bSupported;
}


float FIOSChunkInstall::GetChunkProgress( uint32 ChunkID, EChunkProgressReportingType::Type ReportType )
{
    float Val;
    switch(ReportType)
    {
        case EChunkProgressReportingType::PercentageComplete:
            Val = GetChunkPercentComplete(ChunkID);
            break;
        default:
            UE_LOGF(LogChunkInstaller, Error, "Unsupported ProgressReportType: %i", (int)ReportType);
            Val = 0.0f;
            break;
    }
    return Val;
}


float FIOSChunkInstall::GetChunkPercentComplete(uint32 ChunkID)
{
    // early out if we already know it's local
    const FIOSChunkStatus* CurStatus = ChunkStatus.Find(ChunkID);
    
    if (CurStatus)
    {
        return ((float)CurStatus->Request.progress.fractionCompleted) * 100.0f;
    }
    
    return 0.0f;
}


EChunkInstallSpeed::Type FIOSChunkInstall::GetInstallSpeed()
{
    // TODO: utilize the NSProgress of all requests to determine if we are paused, slow, or fast
    
    return EChunkInstallSpeed::Fast;
}


bool FIOSChunkInstall::SetInstallSpeed( EChunkInstallSpeed::Type InstallSpeed )
{
    // TODO: speed doesn't matter for IOS, but use the NSProgress to pause if not already complete
    
    return true;
}


bool FIOSChunkInstall::PrioritizeChunk( uint32 ChunkID, EChunkPriority::Type Priority )
{
    // create the request
    FString RequestIDs = FString::Printf(TEXT("Chunk%d"), ChunkID);
    NSString* ID = RequestIDs.GetNSString();
    NSSet<NSString* > * tags = [[NSSet alloc] initWithObjects:&ID count:1];
    NSBundleResourceRequest* ChunkRequest = [[NSBundleResourceRequest alloc] initWithTags:tags];
    
    // set the priority
    ChunkRequest.loadingPriority = 1.0f - (float)Priority/2.0f;
    
    // create the chunk status
    FIOSChunkStatus Status;
    Status.Request = ChunkRequest;
    ChunkStatus.Add(ChunkID, Status);
    
    // start the download
    [ChunkRequest beginAccessingResourcesWithCompletionHandler:^(NSError* error)
     {
        if (error)
        {
            // TODO: handle error
            NSLog(@"Error: %@", error);
        }
        else
        {
            // mount the pak file in the NSBundle
            if (!MountedChunks.Find(ChunkID))
            {
                FIOSChunkStatus* FoundStatus = ChunkStatus.Find(ChunkID);
                NSBundle* Bundle = FoundStatus->Request.bundle;
                FString PakFileName = FString::Printf(TEXT("pakchunk%i-ios.pak"), ChunkID);
                NSString* ResourcePath = [Bundle pathForResource: PakFileName.GetNSString() ofType:nil];
                NSLog(@"ResourcePath: %@", ResourcePath);
                IPakFile* PakFile = FCoreDelegates::MountPak.Execute(FString(ResourcePath), 0);
                if (PakFile)
                {
                    MountedChunks.Add(ChunkID);
					InstallDelegate.Broadcast(ChunkID, true);
                }
                else
                {
                    UE_LOGF(LogChunkInstaller, Warning, "NSBundle Chunk %i couldn't be mounted.", ChunkID);
					InstallDelegate.Broadcast(ChunkID, false);
                }
            }
        }
     }];
    return true;
}


bool FIOSChunkInstall::DebugStartNextChunk()
{
#if !UE_BUILD_SHIPPING
    static uint32 NextChunkID = 0;
    
    //cannot call this function in a submission build.  Will fail cert.
    // create a request and ask for the bundle
#endif
    return true;
}
