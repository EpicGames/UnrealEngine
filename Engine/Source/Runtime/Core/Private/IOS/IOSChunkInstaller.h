// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <Foundation/NSBundle.h>
#include "GenericPlatform/GenericPlatformChunkInstall.h"
#include "Containers/Map.h"

struct FIOSChunkStatus
{
    NSBundleResourceRequest* Request;
};

/**
 * IOS implementation of FGenericPlatformChunkInstall.
 */
class CORE_API FIOSChunkInstall : public FGenericPlatformChunkInstall
{
public:
    
    FIOSChunkInstall();
    virtual ~FIOSChunkInstall();
    
    /**
     * Check if a given reporting type is supported.
     *
     * @param ReportType Enum specifying how progress is reported.
     * @return true if reporting type is supported on the current platform.
     */
    virtual bool GetProgressReportingTypeSupported(EChunkProgressReportingType::Type ReportType) override;
    
    /**
     * Get the current install progress of a chunk.  Let the user specify report type for platforms that support more than one.
     *
     * @param ChunkID The id of the chunk to check.
     * @param ReportType The type of progress report you want.
     * @return A value whose meaning is dependent on the ReportType param.
     */
    virtual float GetChunkProgress( uint32 ChunkID, EChunkProgressReportingType::Type ReportType ) override;
    
    /**
     * Inquire about the priority of chunk installation vs. game IO.
     *
     * @return Paused, low or high priority.
     */
    virtual EChunkInstallSpeed::Type GetInstallSpeed() override;
    
    /**
     * Specify the priority of chunk installation vs. game IO.
     *
     * @param InstallSpeed Pause, low or high priority.
     * @return false if the operation is not allowed, otherwise true.
     */
    virtual bool SetInstallSpeed( EChunkInstallSpeed::Type InstallSpeed ) override;
    
    /**
     * Hint to the installer that we would like to prioritize a specific chunk
     *
     * @param ChunkID The id of the chunk to prioritize.
     * @param Priority The priority for the chunk.
     * @return false if the operation is not allowed or the chunk doesn't exist, otherwise true.
     */
    virtual bool PrioritizeChunk( uint32 ChunkID, EChunkPriority::Type Priority ) override;
    
    /**
     * Debug function to start transfer of next chunk in the transfer list.  When in PlayGo HostFS emulation, this is the only
     * way moving out of network-local will happen for a chunk.	 Does nothing in a shipping build.
     *
     * @return true if the operation succeeds.
     */
    virtual bool DebugStartNextChunk( ) override;

private:

	/**
	 * Get the current location of a chunk.
	 *
	 * @param ChunkID The id of the chunk to check.
	 * @return Enum specifying whether the chunk is available to use, waiting to install, or does not exist.
	 */
	virtual EChunkLocation::Type GetChunkLocation(uint32 ChunkID) override;
    
    /**
     * Get the current location of a chunk.
     *
     * @param ChunkID The id of the chunk to check.
     * @return Enum specifying whether the chunk is available to use, waiting to install, or does not exist.
     */
    virtual EChunkLocation::Type GetChunkLocation( uint32 ChunkID, bool bSkipMount );
    
    /**
     * Get the current percent downloaded of a chunk.
     *
     * @param ChunkID The id of the chunk to check.
     * @return Percent downloaded in 99.99 format.
     */
    float GetChunkPercentComplete(uint32 ChunkID);
    
    /** Unload PlayGo and free everything up */
    void ShutDown();
    
private:
    
    /** map of chunk IDs to the cached status of that chunk */
    TMap<uint32, FIOSChunkStatus> ChunkStatus;
    TSet<int32> MountedChunks;
};
