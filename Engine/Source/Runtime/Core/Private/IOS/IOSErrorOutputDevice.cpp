// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	IOSPlatformOutputDevices.mm: iOS implementations of OutputDevices functions
=============================================================================*/

#include "IOS/IOSErrorOutputDevice.h"
#include "HAL/PlatformMisc.h"
#include "HAL/PlatformAtomics.h"
#include "Misc/OutputDeviceHelper.h"
#include "Misc/OutputDeviceRedirector.h"
#include "CoreGlobals.h"
#include <cxxabi.h>
#import <Foundation/Foundation.h>

FIOSErrorOutputDevice::FIOSErrorOutputDevice()
:	ErrorPos(0)
{
}

void FIOSErrorOutputDevice::Serialize( const TCHAR* Msg, ELogVerbosity::Type Verbosity, const class FName& Category )
{
	FPlatformMisc::LowLevelOutputDebugString(*FOutputDeviceHelper::FormatLogLine(Verbosity, Category, Msg, GPrintLogTimes));	if( GIsGuarded )
	{
		UE_DEBUG_BREAK();
	}
	else
	{
		// We crashed outside the guarded code (e.g. appExit).
		HandleError();
		FPlatformMisc::RequestExit( true, TEXT("FIOSErrorOutputDevice::Serialize"));
	}
}

void FIOSErrorOutputDevice::HandleError()
{
	// make sure we don't report errors twice
	static int32 CallCount = 0;
	int32 NewCallCount = FPlatformAtomics::InterlockedIncrement(&CallCount);
	if (NewCallCount != 1)
	{
		UE_LOGF(LogIOS, Error, "HandleError re-entered." );
		return;
	}

	GIsGuarded = 0;
	GIsRunning = 0;
	GIsCriticalError = 1;
	GLogConsole = NULL;
	GErrorHist[UE_ARRAY_COUNT(GErrorHist) - 1] = 0;
    
	// Dump the error and flush the log.
#if !NO_LOGGING
	NSArray<NSString *>* CallStackSymbols = [NSThread callStackSymbols];
	int32 Status = 0;
	static_assert(sizeof(GErrorHist[0]) == 2 * sizeof(char));
	int Pos = 0;	// In unit of TCHAR, aka wide char
	int NumbersOfLinesToSkip = 5; // First 5 lines of callstacks are just error output stuff
	for (NSString* Line in CallStackSymbols)
	{
		if ([CallStackSymbols indexOfObject:Line] < NumbersOfLinesToSkip)
		{
			continue;
		}
		if (Pos >= sizeof(GErrorHist))
		{
			break;
		}
		// Try to demangle the function name
		NSMutableArray<NSString *>* SubStrings = [NSMutableArray arrayWithArray:[Line componentsSeparatedByString:@" "]];
		// last part of the string should look like "_ZN21FIOSErrorOutputDevice11HandleErrorEv + 248"
		const int NameIndex = 3;	// function name is 3rd from last
		if (SubStrings.count >= NameIndex && [SubStrings[SubStrings.count - NameIndex + 1] isEqualToString:@"+"])
		{
			char* DemangledName = abi::__cxa_demangle([SubStrings[SubStrings.count - NameIndex] cStringUsingEncoding:NSUTF8StringEncoding], NULL, NULL, &Status);
			if (Status >= 0)
			{
				SubStrings[SubStrings.count - NameIndex] = [NSString stringWithCString:DemangledName encoding:NSUTF8StringEncoding];
			}
			free(DemangledName);
		}
		Line = [SubStrings componentsJoinedByString:@" "];
		// NSString does not understand wide CString
		[Line getCString:(char *)(GErrorHist + Pos)
			   maxLength:(sizeof(GErrorHist) - Pos) * 2
				encoding:NSUTF16StringEncoding];
		Pos += [Line lengthOfBytesUsingEncoding:NSUTF16StringEncoding] / 2;
		// Append L'\n' instead of L'\0'
		*(GErrorHist + Pos) = L'\n';
		Pos++;
	}
	
	FDebug::LogFormattedMessageWithCallstack(LogIOS.GetCategoryName(), __FILE__, __LINE__, TEXT("=== Critical error: ==="), GErrorHist, ELogVerbosity::Error);
#endif

	GLog->Panic();
}
