// Copyright Epic Games, Inc. All Rights Reserved.

#include "IOS/IOSMallocZone.h"
#include "IOS/IOSPlatformMisc.h"
#include "IOS/IOSPlatformCrashContext.h"
#include "HAL/PlatformTLS.h"
#include "HAL/PlatformProcess.h"
#include <malloc/malloc.h>

FIOSMallocZone::FIOSMallocZone( uint64 const InitialSize )
: MemoryZone( malloc_create_zone( InitialSize, 0 ) )
{
}

FIOSMallocZone::~FIOSMallocZone()
{
	if ( MemoryZone )
	{
		malloc_destroy_zone( MemoryZone );
	}
}

void* FIOSMallocZone::Malloc( SIZE_T Size, uint32 Alignment )
{
	check( MemoryZone );
	void* Result = malloc_zone_malloc( MemoryZone, Size );
	return Result;
}

void* FIOSMallocZone::Realloc( void* Ptr, SIZE_T NewSize, uint32 Alignment )
{
	check( MemoryZone );
	void* Result = malloc_zone_realloc( MemoryZone, Ptr, NewSize );
	return Result;
}

void FIOSMallocZone::Free( void* Ptr )
{
	check( MemoryZone );
	malloc_zone_free( MemoryZone, Ptr );
}

bool FIOSMallocZone::GetAllocationSize(void *Original, SIZE_T &SizeOut)
{
	SizeOut = malloc_size( Original );
	return SizeOut > 0;
}

bool FIOSMallocZone::IsInternallyThreadSafe() const
{
	return true;
}

bool FIOSMallocZone::ValidateHeap()
{
	check( MemoryZone );
	return malloc_zone_check( MemoryZone ) ;
}

const TCHAR* FIOSMallocZone::GetDescriptiveName()
{
	return TEXT("MallocZone");
}

FIOSMallocCrashHandler::FIOSMallocCrashHandler( uint64 const InitialSize )
: FIOSMallocZone( InitialSize )
, OriginalHeap(UE::Private::GMalloc)
, CrashContext( nullptr )
, ThreadId( -1 )
{
	check(UE::Private::GMalloc);
}

FIOSMallocCrashHandler::~FIOSMallocCrashHandler()
{
	// We crashed, so don't try and tidy the malloc zone
	if ( ThreadId != -1 )
	{
		MemoryZone = nullptr;
	}
}
	
void FIOSMallocCrashHandler::Enable( FIOSCrashContext* Context, uint32 CrashedThreadId )
{
	check(Context);
	CrashContext = Context;
	ThreadId = CrashedThreadId;
	OriginalHeap = UE::Private::GMalloc;
	if (PLATFORM_USES_FIXED_GMalloc_CLASS && GFixedMallocLocationPtr)
	{
		*GFixedMallocLocationPtr = this; // replace fixed allocator with crash allocator
	}
	UE::Private::GMalloc = this;
}
	
void FIOSMallocCrashHandler::SetContext( FIOSCrashContext* Context )
{
	check(Context);
	CrashContext = Context;
}

void* FIOSMallocCrashHandler::Malloc( SIZE_T Size, uint32 Alignment )
{
	void* Result = nullptr;
	if ( IsOnCrashedThread() )
	{
		Result = FIOSMallocZone::Malloc( Size, Alignment );
		if( !Result )
		{
			check(CrashContext);
			CrashContext->GenerateCrashInfo();
		}
	}
	return Result;
}

void* FIOSMallocCrashHandler::Realloc( void* Ptr, SIZE_T NewSize, uint32 Alignment )
{
	void* Result = nullptr;
	if ( IsOnCrashedThread() )
	{
		if ( Ptr == nullptr || MemoryZone->size( MemoryZone, Ptr ) > 0 )
		{
			Result = FIOSMallocZone::Realloc( Ptr, NewSize, Alignment );
			if( NewSize && !Result )
			{
				check(CrashContext);
				CrashContext->GenerateCrashInfo();
			}
		}
		else if ( NewSize )
		{
			if ( FCStringWide::Strcmp( OriginalHeap->GetDescriptiveName(), TEXT("ANSI") ) != 0 )
			{
				SIZE_T OldSize = 0;
				if ( OriginalHeap->GetAllocationSize( Ptr, OldSize ) )
				{
					Result = Malloc( NewSize, Alignment );
					if ( Result )
					{
						FMemory::Memcpy( Result, Ptr, FMath::Min( NewSize, OldSize ) );
					}
					else
					{
						check(CrashContext);
						CrashContext->GenerateCrashInfo();
					}
				}
			}
			else // Can't safely handle this, so just report & exit
			{
				check(CrashContext);
				CrashContext->GenerateCrashInfo();
			}
		}
	}
	return Result;
}

void FIOSMallocCrashHandler::Free( void* Ptr )
{
	if ( IsOnCrashedThread() && Ptr && MemoryZone->size( MemoryZone, Ptr ) > 0 )
	{
		FIOSMallocZone::Free( Ptr );
	}
}

bool FIOSMallocCrashHandler::GetAllocationSize( void *Original, SIZE_T &SizeOut )
{
	SizeOut = 0;
	if ( IsOnCrashedThread() && Original )
	{
		SizeOut = MemoryZone->size( MemoryZone, Original );
	}
	return SizeOut > 0;
}

const TCHAR* FIOSMallocCrashHandler::GetDescriptiveName()
{
	return TEXT("MallocCrashHandler");
}

bool FIOSMallocCrashHandler::IsOnCrashedThread( void )
{
	// Suspend threads other than the crashed one to prevent serious memory errors.
	// Only the crashed thread can do anything meaningful from here anyway.
	if ( ThreadId == FPlatformTLS::GetCurrentThreadId() )
	{
		return true;
	}
	else
	{
		FPlatformProcess::SleepInfinite();
	}
}
