// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProfilingDebugging/MemoryTrace.h"

#if UE_MEMORY_TRACE_ENABLED

#include "ProfilingDebugging/CallstackTrace.h"
#include "ProfilingDebugging/TraceMalloc.h"
#include "Containers/StringView.h"
#include "CoreTypes.h"
#include "HAL/MemoryBase.h"
#include "HAL/Platform.h"
#include "HAL/PlatformTime.h"
#include "Misc/CoreDelegates.h"
#include "Misc/CString.h"
#include "Trace/Trace.inl"

#import <Foundation/Foundation.h>


////////////////////////////////////////////////////////////////////////////////
FMalloc* MemoryTrace_CreateInternal(FMalloc*, int, const ANSICHAR* const*);

////////////////////////////////////////////////////////////////////////////////
FMalloc* MemoryTrace_Create(FMalloc* InMalloc)
{
    FMalloc* Result = InMalloc;
    
    NSArray* DocumentsPaths = NSSearchPathForDirectoriesInDomains(NSDocumentDirectory, NSUserDomainMask, YES);
    NSString* DocumentsDirectory = [DocumentsPaths objectAtIndex:0];
    NSString* DocsCmdLinePath = [[DocumentsDirectory stringByAppendingPathComponent:@"uecommandline.txt"] retain];
    NSString* BundleCmdLinePath = [[[[NSBundle mainBundle] bundlePath] stringByAppendingPathComponent:@"uecommandline.txt"] retain];
    NSFileManager* FileManager = [NSFileManager defaultManager];
    
    // try Documents folder and if there is no command line there, try the one in the bundle
    NSString* CmdLinePaths[] = { DocsCmdLinePath, BundleCmdLinePath };
    
    for (int i=0;i<2;i++)
    {
        if ([FileManager fileExistsAtPath:CmdLinePaths[i]])
        {
            NSError* Error = nil;
            NSString* CmdLine = [[NSString stringWithContentsOfFile:CmdLinePaths[i] encoding:NSUTF8StringEncoding error:&Error] retain];
            if (Error == nil)
            {
                int32 ArgC = 2;
                const char* ArgV[] = { "UE5", (char*)[CmdLine UTF8String] };
                Result = MemoryTrace_CreateInternal(InMalloc, ArgC, ArgV);
                [CmdLine release];
                break;
            }
        }
    }
    
    [DocsCmdLinePath release];
    [BundleCmdLinePath release];

    return Result;
}

#endif // UE_MEMORY_TRACE_ENABLED
