// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "Apple/AppleStringUtils.h"

#if UE_BUILD_SHIPPING

void Modules_Initialize() {}

#else // !UE_BUILD_SHIPPING

#include "HAL/LowLevelMemTracker.h"
#include "HAL/PlatformProcess.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/Paths.h"
#include "ProfilingDebugging/ModuleDiagnostics.h"
#include "ProfilingDebugging/MemoryTrace.h"
#include "ProfilingDebugging/MetadataTrace.h"

#include "Apple/PreAppleSystemHeaders.h"
#include <mach/mach.h>
#include <mach-o/dyld_images.h>
#include <mach-o/loader.h>
#include "Apple/PostAppleSystemHeaders.h"

//#include "CoreGlobals.h"
//#define TRACE_MODULES_DEBUG_LOG(Format, ...) \
//	{ \
//		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("[Modules]") Format TEXT("\n"), ##__VA_ARGS__); \
//		UE_LOG(LogTemp, Display, TEXT("[Modules]") Format, ##__VA_ARGS__); \
//	}
#define TRACE_MODULES_DEBUG_LOG(Format, ...)

void Modules_Initialize()
{
	using namespace UE::Trace;

	constexpr uint32 SizeOfSymbolFormatString = 4;
	UE_TRACE_LOG(Diagnostics, ModuleInit, ModuleChannel, sizeof(ANSICHAR) * SizeOfSymbolFormatString)
		<< ModuleInit.SymbolFormat("psym", SizeOfSymbolFormatString)
		<< ModuleInit.ModuleBaseShift(uint8(0));

	struct task_dyld_info dyld_info;
	mach_msg_type_number_t count = TASK_DYLD_INFO_COUNT;
	if (task_info(mach_task_self(), TASK_DYLD_INFO, (task_info_t)&dyld_info, &count) == KERN_SUCCESS)
	{
#if UE_MEMORY_TRACE_ENABLED
		HeapId ProgramHeapId = MemoryTrace_HeapSpec(EMemoryTraceRootHeap::SystemMemory, TEXT("Program"), EMemoryTraceHeapFlags::NeverFrees);
		FString ExecutablePath = FPaths::GetPath(FString([[NSBundle mainBundle]executablePath]));
#endif // UE_MEMORY_TRACE_ENABLED

		const struct dyld_all_image_infos* infos = (const struct dyld_all_image_infos*)dyld_info.all_image_info_addr;
		TRACE_MODULES_DEBUG_LOG(TEXT("Found %d modules"), infos->infoArrayCount);

		// Send Mach-O's uuid as the BuildId. Psym generation seems to have an extra 0 but we will ignore it on the other end.
		constexpr uint32 BuildIdSize = 16;
		uint8 BuildId[BuildIdSize] = {0};

		for (int ImageInfoIndex = 0; ImageInfoIndex < infos->infoArrayCount; ImageInfoIndex++)
		{
			const struct dyld_image_info* Image = &infos->infoArray[ImageInfoIndex];
			TRACE_MODULES_DEBUG_LOG(TEXT("Found module %s"), ANSI_TO_TCHAR(Image->imageFilePath));

			const uint64 ModuleBase = uint64(Image->imageLoadAddress);
			TRACE_MODULES_DEBUG_LOG(TEXT("    Base: 0x%llX"), ModuleBase);

			const struct mach_header_64* Header = (const struct mach_header_64*)Image->imageLoadAddress;

			// calc image size by adding header and size of segments
			uint64 ImageSize = sizeof(*Header) + Header->sizeofcmds;
			constexpr uint64 PageMask = (1 << 12) - 1;
			ImageSize = (ImageSize + PageMask) & ~PageMask; // 4K page aligned
			TRACE_MODULES_DEBUG_LOG(TEXT("    Header: %llu + %llu (%d cmds) -> %llu"), sizeof(*Header), Header->sizeofcmds, Header->ncmds, ImageSize);

			char* CmdPtr = (char*)(Header + 1);
			for (int CommandIndex = 0; CommandIndex < Header->ncmds; ++CommandIndex)
			{
				const struct load_command* LoadCommand = (const struct load_command*)CmdPtr;
				if (LoadCommand->cmd == LC_SEGMENT_64)
				{
					const struct segment_command_64* Segment = (const struct segment_command_64*)LoadCommand;
					TRACE_MODULES_DEBUG_LOG(TEXT("    LC_SEGMENT_64 %s (vmaddr=0x%llX, vmsize=%llu, filesize=%llu)"),
						ANSI_TO_TCHAR(Segment->segname), uint64(Segment->vmaddr), uint64(Segment->vmsize), uint64(Segment->filesize));

					if (uint64(Segment->vmaddr) != 0 && // skips __PAGEZERO segment (4 GiB; reserved virtual memory)
						FPlatformString::Strcmp(Segment->segname, "__LINKEDIT") != 0) // skips __LINKEDIT segment
					{
						ImageSize += uint64(Segment->vmsize);
					}
				}
				else if (LoadCommand->cmd == LC_UUID)
				{
					FMemory::Memcpy(BuildId, ((const struct uuid_command*)LoadCommand)->uuid, BuildIdSize);
				}
				CmdPtr = CmdPtr + LoadCommand->cmdsize;
			}

			TRACE_MODULES_DEBUG_LOG(TEXT("    ImageSize: %llu (end=0x%llX)"), ImageSize, ModuleBase + ImageSize);

			FString ImageName(Image->imageFilePath);

#if UE_MEMORY_TRACE_ENABLED
			bool bInsideExecutablePath = ImageName.StartsWith(ExecutablePath);
#endif // UE_MEMORY_TRACE_ENABLED

			// trim path to leave just image name
			ImageName = FPaths::GetCleanFilename(ImageName);

			const uint32 ModuleSize = uint32(ImageSize);

			UE_TRACE_LOG(Diagnostics, ModuleLoad, ModuleChannel, sizeof(TCHAR) * ImageName.Len() + BuildIdSize)
				<< ModuleLoad.Name(*ImageName, ImageName.Len())
				<< ModuleLoad.Base(ModuleBase)
				<< ModuleLoad.Size(ModuleSize)
				<< ModuleLoad.ImageId(BuildId, BuildIdSize);

#if UE_MEMORY_TRACE_ENABLED
			// Only count the main executable and any other libraries inside our bundle as "Program Size"
			if (bInsideExecutablePath)
			{
				UE_TRACE_METADATA_CLEAR_SCOPE();
				LLM(UE_MEMSCOPE(ELLMTag::ProgramSize));
				MemoryTrace_Alloc(ModuleBase, ImageSize, 1);
				MemoryTrace_MarkAllocAsHeap(ModuleBase, ProgramHeapId);
				MemoryTrace_Alloc(ModuleBase, ImageSize, 1);
			}
#endif // UE_MEMORY_TRACE_ENABLED
		}
	}
}

#undef TRACE_MODULES_DEBUG_LOG

#endif // !UE_BUILD_SHIPPING
