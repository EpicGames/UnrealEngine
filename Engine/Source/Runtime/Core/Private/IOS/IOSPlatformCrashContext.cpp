// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	IOSPlatformCrashContext.cpp: implementations of IOS platform crash context.
=============================================================================*/
#include "IOS/IOSPlatformCrashContext.h"
#include "Apple/ScopeAutoreleasePool.h"
#include "Apple/AppleStringUtils.h"

#if PLATFORM_MAC
#include "Mac/MacSystemIncludes.h"
#elif PLATFORM_IOS
#include "IOS/IOSSystemIncludes.h"
#endif

#include "IOS/IOSPlatformPLCrashReporterIncludes.h"

#include "HAL/ExceptionHandling.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformOutputDevices.h"

#include "Internationalization/Culture.h"
#include "Internationalization/Internationalization.h"

#include "Misc/App.h"
#include "Misc/EngineVersion.h"
#include "Misc/Paths.h"


FIOSApplicationInfo GIOSAppInfo;

#if !PLATFORM_TVOS && !PLATFORM_VISIONOS
PLCrashReporter* FIOSApplicationInfo::CrashReporter;
#endif
FIOSMallocCrashHandler* FIOSApplicationInfo::CrashMalloc;

TFunction<void(char const* const InfoFolder)> FIOSCrashContext::GenerateScreenshotFunc = nullptr;

FIOSCrashContext::FIOSCrashContext(ECrashContextType InType, const TCHAR* InErrorMessage)
	: FApplePlatformCrashContext(InType, InErrorMessage)
{
}

void FIOSCrashContext::CopyMinidump(char const* OutputPath, char const* InputPath) const
{
#if !PLATFORM_TVOS && !PLATFORM_VISIONOS
    int ReportFile = open(OutputPath, O_CREAT|O_WRONLY, 0766);
    int DumpFile = open(InputPath, O_RDONLY, 0766);
    if (ReportFile != -1 && DumpFile != -1)
    {
        char Data[PATH_MAX];
        
        int Bytes = 0;
        while((Bytes = read(DumpFile, Data, PATH_MAX)) > 0)
        {
            write(ReportFile, Data, Bytes);
        }
        
        close(DumpFile);
        close(ReportFile);
        
        unlink(InputPath);
    }
	
	//Make sure we close file handles in case we couldn't open both above
	if (DumpFile != -1)
	{
		close(DumpFile);
	}
	if (ReportFile != -1)
	{
		close(ReportFile);
	}
#endif
}

void FIOSCrashContext::SetGenerateScreenshotFunc(const TFunction<void (const char *const)>& InGenerateScreenshotFunc)
{
    FIOSCrashContext::GenerateScreenshotFunc = InGenerateScreenshotFunc;
}

void FIOSCrashContext::GenerateInfoInFolder(char const* const InfoFolder, bool bIsEnsure) const
{
    // create a crash-specific directory
    char CrashInfoFolder[PATH_MAX] = {};
    FCStringAnsi::Strncpy(CrashInfoFolder, InfoFolder, PATH_MAX);
    
    if(!mkdir(CrashInfoFolder, 0766))
    {
        char FilePath[PATH_MAX] = {};
        FCStringAnsi::Strncpy(FilePath, CrashInfoFolder, PATH_MAX);
        FCStringAnsi::StrncatTruncateDest(FilePath, PATH_MAX, "/report.wer");
        int ReportFile = open(FilePath, O_CREAT|O_WRONLY, 0766);
        if (ReportFile != -1)
        {
            // write BOM
            static uint16 ByteOrderMarker = 0xFEFF;
            write(ReportFile, &ByteOrderMarker, sizeof(ByteOrderMarker));
            
            WriteUTF16String(ReportFile, TEXT("\r\nAppPath="));
            WriteUTF16String(ReportFile, *GIOSAppInfo.AppPath);
            WriteLine(ReportFile, TEXT("\r\n"));
            
            close(ReportFile);
        }
		
		// generate additional files from providers.
		FAdditionalCrashContextParams Params =
		{
			.CrashedThreadId = GetCrashedThreadId()
		};
		DumpAdditionalContext(*FString(CrashInfoFolder), Params);
                
        // generate "minidump" (Apple crash log format)
        FCStringAnsi::Strncpy(FilePath, CrashInfoFolder, PATH_MAX);
        FCStringAnsi::StrncatTruncateDest(FilePath, PATH_MAX, "/minidump.dmp");
		ConvertMinidump(FilePath, GIOSAppInfo.PLCrashReportPath);
        
        // generate "info.txt" custom data for our server
        FCStringAnsi::Strncpy(FilePath, CrashInfoFolder, PATH_MAX);
        FCStringAnsi::StrncatTruncateDest(FilePath, PATH_MAX, "/info.txt");
        ReportFile = open(FilePath, O_CREAT|O_WRONLY, 0766);
        if (ReportFile != -1)
        {
            WriteUTF16String(ReportFile, TEXT("GameName UE5-"));
            WriteLine(ReportFile, *GIOSAppInfo.AppName);
            
            WriteUTF16String(ReportFile, TEXT("BuildVersion 1.0."));
            WriteUTF16String(ReportFile, ItoTCHAR(FEngineVersion::Current().GetChangelist() >> 16, 10));
            WriteUTF16String(ReportFile, TEXT("."));
            WriteLine(ReportFile, ItoTCHAR(FEngineVersion::Current().GetChangelist() & 0xffff, 10));
            
            WriteUTF16String(ReportFile, TEXT("CommandLine "));
            WriteLine(ReportFile, *GIOSAppInfo.CommandLine);
            
            WriteUTF16String(ReportFile, TEXT("BaseDir "));
            WriteLine(ReportFile, *GIOSAppInfo.BranchBaseDir);
            
            WriteUTF16String(ReportFile, TEXT("MachineGuid "));
            WriteLine(ReportFile, *GIOSAppInfo.MachineUUID);
            
            close(ReportFile);
        }
        
        // Introduces a new runtime crash context. Will replace all Windows related crash reporting.
        FCStringAnsi::Strncpy(FilePath, CrashInfoFolder, PATH_MAX);
        FCStringAnsi::StrncatTruncateDest(FilePath, PATH_MAX, "/" );
        FCStringAnsi::StrncatTruncateDest(FilePath, PATH_MAX, FGenericCrashContext::CrashContextRuntimeXMLNameA );
        SerializeAsXML(*FString(FilePath));
        
        // copy log
        FCStringAnsi::Strncpy(FilePath, CrashInfoFolder, PATH_MAX);
        FCStringAnsi::StrncatTruncateDest(FilePath, PATH_MAX, "/");
        FCStringAnsi::StrncatTruncateDest(FilePath, PATH_MAX, (!GIOSAppInfo.AppName.IsEmpty() ? GIOSAppInfo.AppNameUTF8 : "Unreal"));
        FCStringAnsi::StrncatTruncateDest(FilePath, PATH_MAX, ".log");
        
        int LogSrc = open(GIOSAppInfo.AppLogPath, O_RDONLY);
        int LogDst = open(FilePath, O_CREAT|O_WRONLY, 0766);
        
        char Data[PATH_MAX] = {};
        int Bytes = 0;
        while((Bytes = read(LogSrc, Data, PATH_MAX)) > 0)
        {
            write(LogDst, Data, Bytes);
        }
        
        // If present, include the crash report config file to pass config values to the CRC
        FCStringAnsi::Strncpy(FilePath, CrashInfoFolder, PATH_MAX);
        FCStringAnsi::StrncatTruncateDest(FilePath, PATH_MAX, "/");
        FCStringAnsi::StrncatTruncateDest(FilePath, PATH_MAX, FGenericCrashContext::CrashConfigFileNameA);
        int ConfigSrc = open(TCHAR_TO_ANSI(GetCrashConfigFilePath()), O_RDONLY);
        int ConfigDst = open(FilePath, O_CREAT | O_WRONLY, 0766);
        
        while ((Bytes = read(ConfigSrc, Data, PATH_MAX)) > 0)
        {
            write(ConfigDst, Data, Bytes);
        }
        
        close(ConfigDst);
        close(ConfigSrc);
        
        close(LogDst);
        close(LogSrc);
        // best effort, so don't care about result: couldn't copy -> tough, no log
    }
    else
    {
        NSLog(@"******* UE5 - Failed to make folder: %s", CrashInfoFolder);
    }
}

void FIOSCrashContext::GenerateCrashInfo() const
{
    // create a crash-specific directory
    char CrashInfoFolder[PATH_MAX] = {};
    FCStringAnsi::Strncpy(CrashInfoFolder, GIOSAppInfo.CrashReportPath, PATH_MAX);
    FCStringAnsi::StrncatTruncateDest(CrashInfoFolder, PATH_MAX, "/CrashReport-UE-");
    FCStringAnsi::StrncatTruncateDest(CrashInfoFolder, PATH_MAX, GIOSAppInfo.AppNameUTF8);
    FCStringAnsi::StrncatTruncateDest(CrashInfoFolder, PATH_MAX, "-pid-");
    FCStringAnsi::StrncatTruncateDest(CrashInfoFolder, PATH_MAX, ItoANSI(getpid(), 10));
    FCStringAnsi::StrncatTruncateDest(CrashInfoFolder, PATH_MAX, "-");
    FCStringAnsi::StrncatTruncateDest(CrashInfoFolder, PATH_MAX, ItoANSI(GIOSAppInfo.RunUUID.A, 16));
    FCStringAnsi::StrncatTruncateDest(CrashInfoFolder, PATH_MAX, ItoANSI(GIOSAppInfo.RunUUID.B, 16));
    FCStringAnsi::StrncatTruncateDest(CrashInfoFolder, PATH_MAX, ItoANSI(GIOSAppInfo.RunUUID.C, 16));
    FCStringAnsi::StrncatTruncateDest(CrashInfoFolder, PATH_MAX, ItoANSI(GIOSAppInfo.RunUUID.D, 16));
        
    const bool bIsEnsure = false;
    GenerateInfoInFolder(CrashInfoFolder, bIsEnsure);
    if (FIOSCrashContext::GenerateScreenshotFunc) FIOSCrashContext::GenerateScreenshotFunc(CrashInfoFolder);
        
    // for IOS we will need to send the report on the next run
    
    // Sandboxed applications re-raise the signal to trampoline into the system crash reporter as suppressing it may fall foul of Apple's Mac App Store rules.
    // @todo Submit an application to the MAS & see whether Apple's reviewers highlight our crash reporting or trampolining to the system reporter.
    if(GIOSAppInfo.bIsSandboxed)
    {
        struct sigaction Action;
        FMemory::Memzero(&Action, sizeof(struct sigaction));
        Action.sa_handler = SIG_DFL;
        sigemptyset(&Action.sa_mask);
        sigaction(SIGQUIT, &Action, NULL);
        sigaction(SIGILL, &Action, NULL);
        sigaction(SIGEMT, &Action, NULL);
        sigaction(SIGFPE, &Action, NULL);
        sigaction(SIGBUS, &Action, NULL);
        sigaction(SIGSEGV, &Action, NULL);
        sigaction(SIGSYS, &Action, NULL);
        sigaction(SIGABRT, &Action, NULL);
        sigaction(SIGTRAP, &Action, NULL);
        
        raise(Signal);
    }
    
    _Exit(0);
}

void FIOSCrashContext::GenerateEnsureInfo() const
{
    // Prevent CrashReportClient from spawning another CrashReportClient.
    const bool bCanRunCrashReportClient = FCString::Stristr( *(GIOSAppInfo.ExecutableName), TEXT( "CrashReportClient" ) ) == nullptr;
    
#if !PLATFORM_TVOS && !PLATFORM_VISIONOS
    if(bCanRunCrashReportClient)
    {
        SCOPED_AUTORELEASE_POOL;
        
        // Write the PLCrashReporter report to the expected location
        NSData* CrashReport = [FIOSApplicationInfo::CrashReporter generateLiveReport];
        [CrashReport writeToFile:[NSString stringWithUTF8String:GIOSAppInfo.PLCrashReportPath] atomically:YES];
        
        // Use a slightly different output folder name to not conflict with a subequent crash
        const FGuid Guid = FGuid::NewGuid();
        FString GameName = FApp::GetProjectName();
        FString EnsureLogFolder = FString(GIOSAppInfo.CrashReportPath) / FString::Printf(TEXT("EnsureReport-%s-%s"), *GameName, *Guid.ToString(EGuidFormats::Digits));
        
        const bool bIsEnsure = true;
        GenerateInfoInFolder(TCHAR_TO_UTF8(*EnsureLogFolder), bIsEnsure);
        
        FString Arguments;
        if (IsInteractiveEnsureMode())
        {
            Arguments = FString::Printf(TEXT("\"%s/\""), *EnsureLogFolder);
        }
        else
        {
            Arguments = FString::Printf(TEXT("\"%s/\" -Unattended"), *EnsureLogFolder);
        }
        
        FString ReportClient = FPaths::ConvertRelativePathToFull(FPlatformProcess::GenerateApplicationPath(TEXT("CrashReportClient"), EBuildConfiguration::Development));
        FPlatformProcess::ExecProcess(*ReportClient, *Arguments, nullptr, nullptr, nullptr);
    }
#endif
}

void FIOSCrashContext::AddPlatformSpecificProperties() const
{
	//Calculate and Add our AdditionalSystemSymbolsVersion
	//Which additional system libraries should be loaded is tied to our OS version
	{
		FString MajorVersion;
		FString MinorVersion;
		
		FPlatformMisc::GetOSVersions(MajorVersion, MinorVersion);
		FString AdditionalSystemSymbolsVersion = FString::Printf(TEXT("[%s %s 64b]" ), *MajorVersion, *MinorVersion);
		
		AddCrashProperty(TEXT("AdditionalSystemSymbolsVersion"), AdditionalSystemSymbolsVersion);
	}
}

void FIOSApplicationInfo::Init()
{
	SCOPED_AUTORELEASE_POOL;
	
	// Prevent clang/ld from dead-code-eliminating the nothrow_t variants of global new.
	// This ensures that all OS calls to operator new go through our allocators and delete cleanly.
	{
		std::nothrow_t t;
		char* d = (char*)(operator new(8, t));
		delete d;
		
		d = (char*)operator new[]( 8, t );
		delete [] d;
	}
	
	AppName = FApp::GetProjectName();
	FCStringAnsi::Strncpy(AppNameUTF8, TCHAR_TO_UTF8(*AppName), PATH_MAX + 1);
	
	ExecutableName = FPlatformProcess::ExecutableName();
	
	AppPath = FString([[NSBundle mainBundle] executablePath]);
	
	AppBundleID = FString([[NSBundle mainBundle] bundleIdentifier]);
	
	NumCores = FPlatformMisc::NumberOfCores();
	
	LCID = FString::Printf(TEXT("%d"), FInternationalization::Get().GetCurrentCulture()->GetLCID());
	
	PrimaryGPU = FPlatformMisc::GetPrimaryGPUBrand();
	
	RunUUID = RunGUID();
	
	NSOperatingSystemVersion CurrentOSXVersion = [[NSProcessInfo processInfo] operatingSystemVersion];
	OSVersion = FString::Printf(TEXT("%ld.%ld.%ld"), CurrentOSXVersion.majorVersion, CurrentOSXVersion.minorVersion, CurrentOSXVersion.patchVersion);
	FCStringAnsi::Strncpy(OSVersionUTF8, TCHAR_TO_UTF8(*OSVersion), PATH_MAX + 1);
	
	// macOS build number is only accessible on non-sandboxed applications as it resides outside the accessible sandbox
	if(!bIsSandboxed)
	{
		NSDictionary* SystemVersion = [NSDictionary dictionaryWithContentsOfFile: @"/System/Library/CoreServices/SystemVersion.plist"];
		OSBuild = FString((NSString*)[SystemVersion objectForKey: @"ProductBuildVersion"]);
	}
	
	char TempSysCtlBuffer[PATH_MAX] = {};
	size_t TempSysCtlBufferSize = PATH_MAX;
	
	sysctlbyname("kern.osrelease", TempSysCtlBuffer, &TempSysCtlBufferSize, NULL, 0);
	BiosRelease = TempSysCtlBuffer;
	uint32 KernelRevision = 0;
	TempSysCtlBufferSize = 4;
	sysctlbyname("kern.osrevision", &KernelRevision, &TempSysCtlBufferSize, NULL, 0);
	BiosRevision = FString::Printf(TEXT("%d"), KernelRevision);
	TempSysCtlBufferSize = PATH_MAX;
	sysctlbyname("kern.uuid", TempSysCtlBuffer, &TempSysCtlBufferSize, NULL, 0);
	BiosUUID = TempSysCtlBuffer;
	TempSysCtlBufferSize = PATH_MAX;
	sysctlbyname("hw.model", TempSysCtlBuffer, &TempSysCtlBufferSize, NULL, 0);
	MachineModel = TempSysCtlBuffer;
	TempSysCtlBufferSize = PATH_MAX+1;
	sysctlbyname("machdep.cpu.brand_string", MachineCPUString, &TempSysCtlBufferSize, NULL, 0);
	
	gethostname(MachineName, UE_ARRAY_COUNT(MachineName));
	
   BranchBaseDir = FString::Printf( TEXT( "%s!%s!%s!%d" ), *FApp::GetBranchName(), FPlatformProcess::BaseDir(), FPlatformMisc::GetEngineMode(), FEngineVersion::Current().GetChangelist() );
	
	// Get the paths that the files will actually have been saved to
	FString LogDirectory = FPaths::ProjectLogDir();
	TCHAR CommandlineLogFile[MAX_SPRINTF]=TEXT("");
	
	// Use the log file specified on the commandline if there is one
	CommandLine = FCommandLine::Get();
	FString LogPath = FGenericPlatformOutputDevices::GetAbsoluteLogFilename();
	LogPath = IFileManager::Get().ConvertToAbsolutePathForExternalAppForWrite(*LogPath);
	FCStringAnsi::Strncpy(AppLogPath, TCHAR_TO_UTF8(*LogPath), PATH_MAX + 1);
	
	// Cache & create the crash report folder.
	FString ReportPath = FPaths::ConvertRelativePathToFull(FString::Printf(TEXT("%s"), *(FPaths::GameAgnosticSavedDir() / TEXT("Crashes"))));
	IFileManager::Get().MakeDirectory(*ReportPath, true);
	ReportPath = IFileManager::Get().ConvertToAbsolutePathForExternalAppForWrite(*ReportPath);
	FCStringAnsi::Strncpy(CrashReportPath, TCHAR_TO_UTF8(*ReportPath), PATH_MAX + 1);
	
	NSString* PLCrashReportFile = [TemporaryCrashReportFolder().GetNSString() stringByAppendingPathComponent:TemporaryCrashReportName().GetNSString()];
	[PLCrashReportFile getCString:PLCrashReportPath maxLength:PATH_MAX encoding:NSUTF8StringEncoding];
}
    
FIOSApplicationInfo::~FIOSApplicationInfo()
{
#if !PLATFORM_TVOS && !PLATFORM_VISIONOS
	if (CrashReporter)
	{
		[CrashReporter release];
		CrashReporter = nil;
	}
#endif
}
    
FGuid FIOSApplicationInfo::RunGUID()
{
	static FGuid Guid;
	if (!Guid.IsValid())
	{
		FPlatformMisc::CreateGuid(Guid);
	}
	return Guid;
}
    
FString FIOSApplicationInfo::TemporaryCrashReportFolder()
{
	static FString PLCrashReportFolder;
	if (PLCrashReportFolder.IsEmpty())
	{
		SCOPED_AUTORELEASE_POOL;
		
		NSArray* Paths = NSSearchPathForDirectoriesInDomains(NSCachesDirectory, NSUserDomainMask, YES);
		NSString* CacheDir = [Paths objectAtIndex: 0];
		
		NSString* BundleID = [[NSBundle mainBundle] bundleIdentifier];
		if (!BundleID)
		{
			BundleID = [[NSProcessInfo processInfo] processName];
		}
		check(BundleID);
		
		NSString* PLCrashReportFolderPath = [CacheDir stringByAppendingPathComponent: BundleID];
		PLCrashReportFolder = FString(PLCrashReportFolderPath);
	}
	return PLCrashReportFolder;
}
    
FString FIOSApplicationInfo::TemporaryCrashReportName()
{
	static FString PLCrashReportFileName(RunGUID().ToString() + TEXT(".plcrash"));
	return PLCrashReportFileName;
}
