// Copyright Epic Games, Inc. All Rights Reserved.

#include "IOS/IOSPlatformFile.h"

#include "Apple/AppleStringUtils.h"
#include "Apple/ScopeAutoreleasePool.h"
#if PLATFORM_MAC
#include "Mac/MacSystemIncludes.h"
#elif PLATFORM_IOS
#include "IOS/IOSSystemIncludes.h"
#endif

#include "Apple/ApplePlatformMisc.h"
#include "HAL/PlatformTLS.h"
#include "Containers/StringConv.h"
#include "HAL/PlatformTime.h"
#include "Templates/Function.h"
#include "HAL/CriticalSection.h"
#include "HAL/PlatformProcess.h"
#include "Misc/Parse.h"
#include "Misc/CoreMisc.h"
#include "Misc/CommandLine.h"
#include "Misc/App.h"
#include "Misc/Paths.h"
#include "Misc/ScopeLock.h"
#include "Misc/ScopeExit.h"
#include "ProfilingDebugging/PlatformFileTrace.h"
#include "HAL/DiskUtilizationTracker.h"

#include "Async/MappedFileHandle.h"

#if PLATFORM_USE_PLATFORM_FILE_MANAGED_STORAGE_WRAPPER
#include "HAL/IPlatformFileManagedStorageWrapper.h"
#endif //PLATFORM_USE_PLATFORM_FILE_MANAGED_STORAGE_WRAPPER

// make an FTimeSpan object that represents the "epoch" for time_t (from a stat struct)
const FDateTime IOSEpoch(1970, 1, 1);

namespace
{
	FFileStatData IOSStatToUEFileData(struct stat& FileInfo)
	{
		const bool bIsDirectory = S_ISDIR(FileInfo.st_mode);

		int64 FileSize = -1;
		if (!bIsDirectory)
		{
			FileSize = FileInfo.st_size;
		}

		return FFileStatData(
			IOSEpoch + FTimespan::FromSeconds(static_cast<double>(FileInfo.st_ctime)),
			IOSEpoch + FTimespan::FromSeconds(static_cast<double>(FileInfo.st_atime)),
			IOSEpoch + FTimespan::FromSeconds(static_cast<double>(FileInfo.st_mtime)), 
			FileSize,
			bIsDirectory,
			!(FileInfo.st_mode & S_IWUSR)
			);
	}
}



/* FIOSFileHandle class
 *****************************************************************************/

/** 
 * Managed IOS file handle implementation which limits number of open files. This
 * is to prevent running out of system file handles (700). Should not be neccessary when 
 * using pak file (e.g., SHIPPING?) so not particularly optimized. Only manages 
 * files which are opened READ_ONLY.
 **/
// @todo: Merge all of the managed file handles into one class!
#define MANAGE_FILE_HANDLES_IOS 1 // !UE_BUILD_SHIPPING

struct FManagedFile
{
	int32 Handle;
	uint32 ID;
	double AccessTime;
};

class FIOSFileHandle : public IFileHandle
{
	static const int32 READWRITE_SIZE = 1024 * 1024;
	static const int32 ACTIVE_HANDLE_COUNT_PER_THREAD = 100;

public:

	FIOSFileHandle( int32 InFileHandle, const FString& InFilename, bool bIsForRead )
		: FileHandle(InFileHandle)
#if !UE_BUILD_SHIPPING || MANAGE_FILE_HANDLES_IOS
		, Filename(InFilename)
#endif
#if MANAGE_FILE_HANDLES_IOS
        , HandleSlot(-1)
        , FileSize(0)
#endif
	{
		check(FileHandle != 0);

#if MANAGE_FILE_HANDLES_IOS

		static uint32 NextID = 1;
		FileID = NextID++;

		// get the per-thread buffers
		ManagedFiles = (FManagedFile*)FPlatformTLS::GetTlsValue(ManagedFilesTlsSlot);

		// not made yet on this thread, make the buffers now
		if (ManagedFiles == NULL)
		{
			ManagedFiles = new FManagedFile[ACTIVE_HANDLE_COUNT_PER_THREAD];
			FMemory::Memzero(ManagedFiles, sizeof(FManagedFile) * ACTIVE_HANDLE_COUNT_PER_THREAD);
			FPlatformTLS::SetTlsValue(ManagedFilesTlsSlot, ManagedFiles);
		}

        // Only files opened for read will be managed
        if( bIsForRead )
        {
            ReserveSlot();
            ManagedFiles[HandleSlot].Handle = FileHandle;

			struct stat FileInfo;
			FileInfo.st_size = -1;
			// check the read path
			fstat(FileHandle, &FileInfo);
			FileSize = FileInfo.st_size;
        }
#endif

		Seek(0);
	}

	/**
	 * Destructor.
	 */
	virtual ~FIOSFileHandle( )
	{
#if MANAGE_FILE_HANDLES_IOS
        if( IsManaged() )
        {
            if( ManagedFiles[HandleSlot].ID == FileID )
            {
                close(FileHandle);
				ManagedFiles[HandleSlot].ID = 0;
            }
        }
        else
#endif
        {
		    close(FileHandle);
        }

		FileHandle = -1;
	}

	int64 InternalRead(uint8* Destination, int64 BytesToRead)
	{
		int64 TotalBytesRead = 0;
		FScopedDiskUtilizationTracker Tracker(BytesToRead, FileOffset);
		TRACE_PLATFORMFILE_BEGIN_READ(this, FileHandle, FileOffset, BytesToRead);
		ON_SCOPE_EXIT
		{
			TRACE_PLATFORMFILE_END_READ(this, TotalBytesRead);
		};

		while (BytesToRead)
		{
			check(BytesToRead >= 0);
			int64 ThisSize = FMath::Min<int64>(READWRITE_SIZE, BytesToRead);
			check(Destination);
			int64 BytesRead = read(FileHandle, Destination, ThisSize);
			TotalBytesRead += BytesRead;
			if (BytesRead != ThisSize)
			{
				return TotalBytesRead;
			}
			Destination += ThisSize;
			BytesToRead -= ThisSize;
		}
		return TotalBytesRead;
	}

public:

	// Begin IFileHandle interface

	virtual bool Read( uint8* Destination, int64 BytesToRead ) override
	{
 #if MANAGE_FILE_HANDLES_IOS
       if( IsManaged() )
        {
		   if (FileOffset == FileSize)
		   {
			   return false;
		   }

            ActivateSlot();
			lseek(FileHandle, FileOffset, SEEK_SET);
			// read into the buffer, and make sure it worked
			int64 BytesRead = InternalRead(Destination, BytesToRead);
			FileOffset += BytesRead;
			if (BytesRead != BytesToRead)
			{
				return FileOffset == FileSize;
			}
			return true;
        }
        else
#endif
        {
			int64 Pos = Tell();
			int64 BytesRead = InternalRead(Destination, BytesToRead);
			if (BytesRead != BytesToRead)
			{
				// If we requested more than was in the file return true if we read all the remaining content
				return BytesRead > 0 && (Pos + BytesRead) == Size();
			}
			return true;
        }
	}

	virtual bool ReadAt(uint8* Destination, int64 BytesToRead, int64 Offset) override
	{
		if (BytesToRead < 0 || Offset < 0)
		{
			return false;
		}

		if (BytesToRead == 0)
		{
			return true;
		}

#if MANAGE_FILE_HANDLES_IOS
		if (IsManaged())
		{
			ActivateSlot();
		}
#endif //MANAGE_FILE_HANDLES_IOS

		int64 TotalBytesRead = 0;
		FScopedDiskUtilizationTracker Tracker(BytesToRead, Offset);
		TRACE_PLATFORMFILE_BEGIN_READ(this, FileHandle, Offset, BytesToRead);
		ON_SCOPE_EXIT
		{
			TRACE_PLATFORMFILE_END_READ(this, TotalBytesRead);
		};

		do
		{
			size_t BytesToRead32 = static_cast<size_t>(FMath::Min<int64>(READWRITE_SIZE, BytesToRead));
			ssize_t BytesRead = pread(FileHandle, Destination, BytesToRead32, Offset);

			if (BytesRead == -1 && errno == EINTR)
			{
				continue;
			}
			if (BytesRead <= 0)
			{
				return false;
			}

			TotalBytesRead += BytesRead;
			Offset += BytesRead;
			BytesToRead -= BytesRead;
			Destination += BytesRead;

		} while (BytesToRead > 0);

		return true;
	}

	virtual bool Seek( int64 NewPosition ) override
	{
		check(NewPosition >= 0);

#if MANAGE_FILE_HANDLES_IOS
        if( IsManaged() )
        {
            FileOffset = NewPosition >= FileSize ? FileSize - 1 : NewPosition;
            return true;
        }
        else
#endif
        {
		    return (lseek(FileHandle, NewPosition, SEEK_SET) != -1);
        }
	}

	virtual bool SeekFromEnd( int64 NewPositionRelativeToEnd = 0 ) override
	{
		check(NewPositionRelativeToEnd <= 0);

#if MANAGE_FILE_HANDLES_IOS
        if( IsManaged() )
        {
            FileOffset = (NewPositionRelativeToEnd >= FileSize) ? 0 : ( FileSize + NewPositionRelativeToEnd - 1 );
            return true;
        }
        else
#endif
        {
		    return lseek(FileHandle, NewPositionRelativeToEnd, SEEK_END) != -1;
        }
	}

	virtual bool Flush(const bool bFullFlush = false) override
	{
#if MANAGE_FILE_HANDLES_IOS
		if (IsManaged())
		{
			return false;
		}
#endif
		if (bFullFlush)
		{
			// iOS needs fcntl with F_FULLFSYNC to guarantee a full flush,
			// but still fallback to fsync if fcntl fails
			if (fcntl(FileHandle, F_FULLFSYNC) == 0)
			{
				return true;
			}
		}
		return fsync(FileHandle) == 0;
	}

	virtual bool Truncate(int64 NewSize) override
	{
#if MANAGE_FILE_HANDLES_IOS
		if (IsManaged())
		{
			return false;
		}
#endif
		int Result = 0;
		do { Result = ftruncate(FileHandle, NewSize); } while (Result < 0 && errno == EINTR);
		return Result == 0;
	}

	virtual int64 Size( ) override
	{
#if MANAGE_FILE_HANDLES_IOS
        if( IsManaged() )
        {
            return FileSize;
        }
        else
#endif
        {
		    return IFileHandle::Size();
        }
	}

	virtual int64 Tell( ) override
	{
#if MANAGE_FILE_HANDLES_IOS
        if( IsManaged() )
        {
            return FileOffset;
        }
        else
#endif
        {
		    return lseek(FileHandle, 0, SEEK_CUR);
        }
	}

	virtual bool Write(const uint8* Source, int64 BytesToWrite) override
	{
		while (BytesToWrite > 0)
		{
			const int64 ThisSize = FMath::Min<int64>(READWRITE_SIZE, BytesToWrite);
			const int64 Written = write(FileHandle, Source, ThisSize);
			if (Written <= 0)
			{
				if (errno == EINTR)
				{
					continue;
				}
				else
				{
					return false;
				}
			}
			check(Written <= ThisSize);
			Source += Written;
			BytesToWrite -= Written;
		}
		return true;
	}

	// End IFileHandle interface

private:

#if MANAGE_FILE_HANDLES_IOS
    FORCEINLINE bool IsManaged()
    {
        return HandleSlot != -1;
    }

    void ActivateSlot()
    {
		static FCriticalSection LockHandles;
		FScopeLock Lock(&LockHandles);

        if( IsManaged() )
        {
            if( ManagedFiles[HandleSlot].ID != FileID )
            {
				ReserveSlot();
                
				FileHandle = open(TCHAR_TO_UTF8(*Filename), O_RDONLY);

				if (FileHandle != -1)
                {
                    ManagedFiles[HandleSlot].Handle = FileHandle;
                }
            }
            else
            {
                ManagedFiles[HandleSlot].AccessTime = FPlatformTime::Seconds();
            }
        }
    }

    void ReserveSlot()
    {
        HandleSlot = -1;

        // Look for non-reserved slot
        for( int32 i = 0; i < ACTIVE_HANDLE_COUNT_PER_THREAD; ++i )
        {
            if( ManagedFiles[i].ID == 0 )
            {
                HandleSlot = i;
                break;
            }
        }

        // Take the oldest handle
        if( HandleSlot == -1 )
        {
            int32 Oldest = 0;
            for( int32 i = 1; i < ACTIVE_HANDLE_COUNT_PER_THREAD; ++i )
            {
                if( ManagedFiles[Oldest].AccessTime > ManagedFiles[i].AccessTime )
                {
                    Oldest = i;
                }
            }

            close( ManagedFiles[Oldest].Handle );
            HandleSlot = Oldest;
        }

        ManagedFiles[HandleSlot].ID = FileID;
        ManagedFiles[HandleSlot].AccessTime = FPlatformTime::Seconds();
    }
#endif

private:

	// Holds the internal file handle.
	int32 FileHandle;

#if !UE_BUILD_SHIPPING || MANAGE_FILE_HANDLES_IOS
	// Holds the name of the file that this handle represents. Kept around for possible reopen of file.
	FString Filename;
#endif

#if MANAGE_FILE_HANDLES_IOS
    // Most recent valid slot index for this handle; >=0 for handles which are managed.
    int32 HandleSlot;

    // Current file offset; valid iff a managed handle.
    int64 FileOffset;

    // Cached file size; valid iff a managed handle.
    int64 FileSize;

    // Each thread keeps a collection of active handles with access times.
    FManagedFile* ManagedFiles;

	// Unique FileID for this file (since handles aren't unique)
	uint32 FileID;

	static int32 ManagedFilesTlsSlot;
#endif
};

#if MANAGE_FILE_HANDLES_IOS
int32 FIOSFileHandle::ManagedFilesTlsSlot = FPlatformTLS::AllocTlsSlot();
#endif




/**
 * iOS File I/O implementation
**/
bool Initialize(IPlatformFile* Inner, const TCHAR* CommandLineParam)
{
	return true;
}

FString FIOSPlatformFile::NormalizeFilename(const TCHAR* Filename)
{
	FString Result(Filename);
	Result.ReplaceInline(TEXT("\\"), TEXT("/"), ESearchCase::CaseSensitive);
	return Result;
}

FString FIOSPlatformFile::NormalizeDirectory(const TCHAR* Directory)
{
	FString Result(Directory);
	Result.ReplaceInline(TEXT("\\"), TEXT("/"), ESearchCase::CaseSensitive);
	return Result;
}


FString FIOSPlatformFile::ConvertToAbsolutePathForExternalAppForRead( const TCHAR* Filename )
{
    struct stat FileInfo;
    FString NormalizedFilename = NormalizeFilename(Filename);
    if (stat(TCHAR_TO_UTF8(*ConvertToPlatformPath(NormalizedFilename, false, false)), &FileInfo) == -1)
    {
        return ConvertToAbsolutePathForExternalAppForWrite(Filename);
    }
    else
    {
        return ConvertToPlatformPath(NormalizedFilename, false, false);
    }
}

FString FIOSPlatformFile::ConvertToAbsolutePathForExternalAppForWrite( const TCHAR* Filename )
{
	struct stat FileInfo;
    FString NormalizedFilename = NormalizeFilename(Filename);
	if (bCreatePublicFiles)
	{
		return ConvertToPlatformPath(NormalizedFilename, true, true);
	}
	else
	{
		return ConvertToPlatformPath(NormalizedFilename, true, false);
	}
}

bool FIOSPlatformFile::FileExists(const TCHAR* Filename)
{
	struct stat FileInfo;
	FString NormalizedFilename = NormalizeFilename(Filename);
	// check the read path
	if (stat(TCHAR_TO_UTF8(*ConvertToPlatformPath(NormalizedFilename, false, false)), &FileInfo) == -1)
	{
		// if not in read path, check the private write path
		if (stat(TCHAR_TO_UTF8(*ConvertToPlatformPath(NormalizedFilename, true, false)), &FileInfo) == -1)
		{
			// if not in the private write path, check the public write path
			if (stat(TCHAR_TO_UTF8(*ConvertToPlatformPath(NormalizedFilename, true, true)), &FileInfo) == -1)
			{
				return false;
			}
		}
	}

	return S_ISREG(FileInfo.st_mode);
}

int64 FIOSPlatformFile::FileSize(const TCHAR* Filename)
{
	struct stat FileInfo;
	FileInfo.st_size = -1;
	FString NormalizedFilename = NormalizeFilename(Filename);
	// check the read path
	if(stat(TCHAR_TO_UTF8(*ConvertToPlatformPath(NormalizedFilename, false, false)), &FileInfo) == -1)
	{
		// if not in read path, check the private write path
		if(stat(TCHAR_TO_UTF8(*ConvertToPlatformPath(NormalizedFilename, true, false)), &FileInfo) == -1)
		{
			// if not in the private write path, check the public write path
			if(stat(TCHAR_TO_UTF8(*ConvertToPlatformPath(NormalizedFilename, true, true)), &FileInfo) == -1)
			{
				return -1;
			}
		}
	}

	// make sure to return -1 for directories
	if (S_ISDIR(FileInfo.st_mode))
	{
		FileInfo.st_size = -1;
	}
	return FileInfo.st_size;
}

bool FIOSPlatformFile::DeleteFile(const TCHAR* Filename)
{
	// only delete from write path

	struct stat FileInfo;
	FileInfo.st_size = -1;
	FString NormalizedFilename = NormalizeFilename(Filename);
	FString IOSPrivateWriteFilename = ConvertToPlatformPath(NormalizedFilename, true, false);
	FString IOSPublicWriteFilename = ConvertToPlatformPath(NormalizedFilename, true, true);

	// Try to delete the file from both the public and private write paths
	bool bDeletedPrivate = unlink(TCHAR_TO_UTF8(*IOSPrivateWriteFilename)) == 0;
	bool bDeletedPublic = unlink(TCHAR_TO_UTF8(*IOSPublicWriteFilename)) == 0;
	
	return bDeletedPrivate || bDeletedPublic;
}

bool FIOSPlatformFile::IsReadOnly(const TCHAR* Filename)
{
	FString NormalizedFilename = NormalizeFilename(Filename);
	FString Filepath = ConvertToPlatformPath(NormalizedFilename, false, false);
	// check read path
	if (access(TCHAR_TO_UTF8(*Filepath), F_OK) == -1)
	{
		// if not in read path, check private write path
		Filepath = ConvertToPlatformPath(NormalizedFilename, true, false);
		if (access(TCHAR_TO_UTF8(*Filepath), F_OK) == -1)
		{
			// if not in private write path, check public write path
			Filepath = ConvertToPlatformPath(NormalizedFilename, true, true);

			if (access(TCHAR_TO_UTF8(*Filepath), F_OK) == -1)
			{
				return false; // file doesn't exist
			}
		}
	}

	if (access(TCHAR_TO_UTF8(*Filepath), W_OK) == -1)
	{
		return errno == EPERM || errno == EACCES;
	}
	return false;
}

bool FIOSPlatformFile::MoveFile(const TCHAR* To, const TCHAR* From)
{
	// move to the write path
	FString ToIOSFilename = ConvertToPlatformPath(NormalizeFilename(To), true, bCreatePublicFiles);
	// move from the read path if the file exists there
	FString FromIOSFilename = ConvertToPlatformPath(NormalizeFilename(From), false, false);
	if (!FileExists(*FromIOSFilename))
	{
		// otherwise try the private write path
		FromIOSFilename = ConvertToPlatformPath(NormalizeFilename(From), true, false);

		if (!FileExists(*FromIOSFilename))
		{
			// and finally try the public write path
			FromIOSFilename = ConvertToPlatformPath(NormalizeFilename(From), true, true);
		}
	}
	return rename(TCHAR_TO_UTF8(*FromIOSFilename), TCHAR_TO_UTF8(*ToIOSFilename)) != -1;
}

bool FIOSPlatformFile::SetReadOnly(const TCHAR* Filename, bool bNewReadOnlyValue)
{
	struct stat FileInfo;
	FString IOSFilename = ConvertToPlatformPath(NormalizeFilename(Filename), false, false);
	if (stat(TCHAR_TO_UTF8(*IOSFilename), &FileInfo) != -1)
	{
		if (bNewReadOnlyValue)
		{
			FileInfo.st_mode &= ~S_IWUSR;
		}
		else
		{
			FileInfo.st_mode |= S_IWUSR;
		}
		return chmod(TCHAR_TO_UTF8(*IOSFilename), FileInfo.st_mode) == 0;
	}
	return false;
}


FDateTime FIOSPlatformFile::GetTimeStamp(const TCHAR* Filename)
{
	// get file times
	struct stat FileInfo;
	FString NormalizedFilename = NormalizeFilename(Filename);
	// check the read path
	if(stat(TCHAR_TO_UTF8(*ConvertToPlatformPath(NormalizedFilename, false, false)), &FileInfo) == -1)
	{
		// if not in the read path, check the private write path
		if(stat(TCHAR_TO_UTF8(*ConvertToPlatformPath(NormalizedFilename, true, false)), &FileInfo) == -1)
		{
			// if not in the private write path, check the public write path
			if(stat(TCHAR_TO_UTF8(*ConvertToPlatformPath(NormalizedFilename, true, true)), &FileInfo) == -1)
			{
				return FDateTime::MinValue();
			}
		}
	}

	// convert _stat time to FDateTime
	FTimespan TimeSinceEpoch(0, 0, FileInfo.st_mtime);
	return IOSEpoch + TimeSinceEpoch;
}

void FIOSPlatformFile::SetTimeStamp(const TCHAR* Filename, const FDateTime DateTime)
{
	// get file times
	struct stat FileInfo;
	FString IOSFilename = ConvertToPlatformPath(NormalizeFilename(Filename), true, false);
	if(stat(TCHAR_TO_UTF8(*IOSFilename), &FileInfo) == -1)
	{
		IOSFilename = ConvertToPlatformPath(NormalizeFilename(Filename), true, true);
		if(stat(TCHAR_TO_UTF8(*IOSFilename), &FileInfo) == -1)
		{
			return;
		}
	}

	// change the modification time only
	struct utimbuf Times;
	Times.actime = FileInfo.st_atime;
	Times.modtime = (time_t)(DateTime - IOSEpoch).GetTotalSeconds();
	utime(TCHAR_TO_UTF8(*IOSFilename), &Times);
}

FDateTime FIOSPlatformFile::GetAccessTimeStamp(const TCHAR* Filename)
{
	// get file times
	struct stat FileInfo;
	FString NormalizedFilename = NormalizeFilename(Filename);
	// check the read path
	if(stat(TCHAR_TO_UTF8(*ConvertToPlatformPath(NormalizedFilename, false, false)), &FileInfo) == -1)
	{
		// if not in the read path, check the private write path
		if(stat(TCHAR_TO_UTF8(*ConvertToPlatformPath(NormalizedFilename, true, false)), &FileInfo) == -1)
		{
			// if not in the private write path, check the public write path
			if(stat(TCHAR_TO_UTF8(*ConvertToPlatformPath(NormalizedFilename, true, true)), &FileInfo) == -1)
			{
				return FDateTime::MinValue();
			}
		}
	}

	// convert _stat time to FDateTime
	FTimespan TimeSinceEpoch(0, 0, FileInfo.st_atime);
	return IOSEpoch + TimeSinceEpoch;
}

FString FIOSPlatformFile::GetFilenameOnDisk(const TCHAR* Filename)
{
	return Filename;
}

FFileStatData FIOSPlatformFile::GetStatData(const TCHAR* FilenameOrDirectory)
{
	struct stat FileInfo;
	FString NormalizedFilename = NormalizeFilename(FilenameOrDirectory);

	// check the read path
	if(stat(TCHAR_TO_UTF8(*ConvertToPlatformPath(NormalizedFilename, false, false)), &FileInfo) == -1)
	{
		// if not in the read path, check the private write path
		if(stat(TCHAR_TO_UTF8(*ConvertToPlatformPath(NormalizedFilename, true, false)), &FileInfo) == -1)
		{
			// if not in the private write path, check the public write path
			if(stat(TCHAR_TO_UTF8(*ConvertToPlatformPath(NormalizedFilename, true, true)), &FileInfo) == -1)
			{
				return FFileStatData();
			}
		}
	}

	return IOSStatToUEFileData(FileInfo);
}

IFileHandle* FIOSPlatformFile::OpenRead(const TCHAR* Filename, bool bAllowWrite)
{
	FString NormalizedFilename = NormalizeFilename(Filename);

	// check the read path
	FString FinalPath = ConvertToPlatformPath(NormalizedFilename, false, false);
	int32 Handle = open(TCHAR_TO_UTF8(*FinalPath), O_RDONLY);
	if(Handle == -1)
	{
		// if not in the read path, check the private write path
		FinalPath = ConvertToPlatformPath(NormalizedFilename, true, false);
		Handle = open(TCHAR_TO_UTF8(*FinalPath), O_RDONLY);
		
		if(Handle == -1)
		{
			// if not in the private write path, check the public write path
			FinalPath = ConvertToPlatformPath(NormalizedFilename, true, true);
			Handle = open(TCHAR_TO_UTF8(*FinalPath), O_RDONLY);
		}
	}

	if (Handle != -1)
	{
		return new FIOSFileHandle(Handle, FinalPath, true);
	}
	return NULL;
}

IFileHandle* FIOSPlatformFile::OpenWrite(const TCHAR* Filename, bool bAppend, bool bAllowRead)
{
	int Flags = O_CREAT;
	if (!bAppend)
	{
		Flags |= O_TRUNC;
	}
	if (bAllowRead)
	{
		Flags |= O_RDWR;
	}
	else
	{
		Flags |= O_WRONLY;
	}
	FString IOSFilename = ConvertToPlatformPath(NormalizeFilename(Filename), true, bCreatePublicFiles);
	int32 Handle = open(TCHAR_TO_UTF8(*IOSFilename), Flags, GetModeFlagsForOpenWrite());

	if (Handle != -1)
	{
		if (!bAppend)
		{
			ftruncate(Handle, 0);
		}

		FIOSFileHandle* FileHandleIOS = new FIOSFileHandle(Handle, IOSFilename, false);
		if (bAppend)
		{
			FileHandleIOS->SeekFromEnd(0);
		}
		return FileHandleIOS;
	}
	return NULL;
}


bool FIOSPlatformFile::DirectoryExists(const TCHAR* Directory)
{
	struct stat FileInfo;
	FString NormalizedDirectory = NormalizeFilename(Directory);
	if (stat(TCHAR_TO_UTF8(*ConvertToPlatformPath(NormalizedDirectory, false, false)), &FileInfo)== -1)
	{
		if (stat(TCHAR_TO_UTF8(*ConvertToPlatformPath(NormalizedDirectory, true, false)), &FileInfo)== -1)
		{
			if (stat(TCHAR_TO_UTF8(*ConvertToPlatformPath(NormalizedDirectory, true, true)), &FileInfo)== -1)
			{
				return false;
			}
		}
	}
	return S_ISDIR(FileInfo.st_mode);
}

bool FIOSPlatformFile::CreateDirectory(const TCHAR* Directory)
{
	FString IOSDirectory = ConvertToPlatformPath(NormalizeFilename(Directory), true, bCreatePublicFiles);
	CFStringRef CFDirectory = FPlatformString::TCHARToCFString(*IOSDirectory);
	bool Result = [[NSFileManager defaultManager] createDirectoryAtPath:(NSString*)CFDirectory withIntermediateDirectories:true attributes:nil error:nil];
	CFRelease(CFDirectory);
	return Result;
}

bool FIOSPlatformFile::DeleteDirectory(const TCHAR* Directory)
{
	FString IOSPrivateWriteDirectory = ConvertToPlatformPath(NormalizeFilename(Directory), true, false);
	FString IOSPublicWriteDirectory = ConvertToPlatformPath(NormalizeFilename(Directory), true, true);
	
	// Try to delete the directory in both the private and public write paths.
	// rmdir returns 0 on success, -1 on failure.
	bool bDeletedPrivate = rmdir(TCHAR_TO_UTF8(*IOSPrivateWriteDirectory)) == 0;
	bool bDeletedPublic = rmdir(TCHAR_TO_UTF8(*IOSPublicWriteDirectory)) == 0;

	return bDeletedPrivate || bDeletedPublic;
}

bool FIOSPlatformFile::IterateDirectory(const TCHAR* Directory, FDirectoryVisitor& Visitor)
{
	SCOPED_AUTORELEASE_POOL;

	const FString DirectoryStr = Directory;
	return IterateDirectoryCommon(Directory, [&](struct dirent* InEntry) -> bool
	{
		// Normalize any unicode forms so we match correctly
		const FString NormalizedFilename = UTF8_TO_TCHAR(([[[NSString stringWithUTF8String:InEntry->d_name] precomposedStringWithCanonicalMapping] cStringUsingEncoding:NSUTF8StringEncoding]));
		const FString FullPath = DirectoryStr / NormalizedFilename;

		return Visitor.CallShouldVisitAndVisit(*FullPath, InEntry->d_type == DT_DIR);
	});
}

bool FIOSPlatformFile::IterateDirectoryStat(const TCHAR* Directory, FDirectoryStatVisitor& Visitor)
{
	SCOPED_AUTORELEASE_POOL;

	const FString DirectoryStr = Directory;
	const FString NormalizedDirectoryStr = NormalizeFilename(Directory);

	return IterateDirectoryCommon(Directory, [&](struct dirent* InEntry) -> bool
	{
		// Normalize any unicode forms so we match correctly
		const FString NormalizedFilename = UTF8_TO_TCHAR(([[[NSString stringWithUTF8String:InEntry->d_name] precomposedStringWithCanonicalMapping] cStringUsingEncoding:NSUTF8StringEncoding]));
		const FString FullPath = DirectoryStr / NormalizedFilename;
		const FString FullNormalizedPath = NormalizedDirectoryStr / NormalizedFilename;

		struct stat FileInfo;

		// check the read path
		if(stat(TCHAR_TO_UTF8(*ConvertToPlatformPath(FullNormalizedPath, false, false)), &FileInfo) == -1)
		{
			// if not in the read path, check the private write path
			if(stat(TCHAR_TO_UTF8(*ConvertToPlatformPath(FullNormalizedPath, true, false)), &FileInfo) == -1)
			{
				// if not in the private write path, check the public write path
				if(stat(TCHAR_TO_UTF8(*ConvertToPlatformPath(FullNormalizedPath, true, true)), &FileInfo) == -1)
				{
					return true;
				}
			}
		}

		return Visitor.CallShouldVisitAndVisit(*FullPath, IOSStatToUEFileData(FileInfo));
	});
}

bool FIOSPlatformFile::DoesCreatePublicFiles()
{
	return bCreatePublicFiles;
}

void FIOSPlatformFile::SetCreatePublicFiles(bool bCreatePublicFilesIn)
{
	bCreatePublicFiles = bCreatePublicFilesIn;
}

FIOSPlatformFile::FIOSPlatformFile()
{
#if FILESHARING_ENABLED
	bCreatePublicFiles = false;
#else
	bCreatePublicFiles = true;
#endif
}

bool FIOSPlatformFile::IterateDirectoryCommon(const TCHAR* Directory, const TFunctionRef<bool(struct dirent*)>& Visitor)
{
	bool Result = false;
	const ANSICHAR* FrameworksPath;
	if (Directory[0] == 0)
	{
		if ([[[[NSBundle mainBundle] bundlePath] pathExtension] isEqual: @"app"])
		{
			FrameworksPath = [[[NSBundle mainBundle] privateFrameworksPath] fileSystemRepresentation];
		}
		else
		{
			FrameworksPath = [[[NSBundle mainBundle] bundlePath] fileSystemRepresentation];
		}
	}

	FString NormalizedDirectory = NormalizeFilename(Directory);
	// If Directory is an empty string, assume that we want to iterate Binaries/Mac (current dir), but because we're an app bundle, iterate bundle's Contents/Frameworks instead
	DIR* Handle = opendir(Directory[0] ? TCHAR_TO_UTF8(*ConvertToPlatformPath(NormalizedDirectory, false, false)) : FrameworksPath);
	if(!Handle)
	{
		// look in the private write file path if it's not in the read file path
		Handle = opendir(Directory[0] ? TCHAR_TO_UTF8(*ConvertToPlatformPath(NormalizedDirectory, true, false)) : FrameworksPath);
		
		if(!Handle)
		{
			// look in the public write file path if it's not in the private write file path
			Handle = opendir(Directory[0] ? TCHAR_TO_UTF8(*ConvertToPlatformPath(NormalizedDirectory, true, true)) : FrameworksPath);
		}
	}
	if (Handle)
	{
		Result = true;
		struct dirent *Entry;
		while (Result && (Entry = readdir(Handle)) != NULL)
		{
			if (FCStringAnsi::Strcmp(Entry->d_name, ".") && FCStringAnsi::Strcmp(Entry->d_name, ".."))
			{
				Result = Visitor(Entry);
			}
		}
		closedir(Handle);
	}
	return Result;
}

FString FIOSPlatformFile::ConvertToPlatformPath(const FString& Filename, bool bForWrite, bool bIsPublicWrite)
{
	FString Result = Filename;
	if (Result.Contains(TEXT("/OnDemandResources/")) || 
		Result.StartsWith(TEXT("/private/var/")) || 
		Result.StartsWith(TEXT("/var/")))
	{
		return Result;
	}

#if !UE_BUILD_SHIPPING
	// My Mac (Designed for iPad) target in xcode: paths already resolved to the container
	// should also be skipped
	static bool bIsIOSAppOnMac = [[NSProcessInfo processInfo] isiOSAppOnMac];
	if (bIsIOSAppOnMac)
	{
		static FString DocumentsBase = FString([NSSearchPathForDirectoriesInDomains(NSDocumentDirectory, NSUserDomainMask, YES) objectAtIndex:0]);
		static FString LibraryBase = FString([NSSearchPathForDirectoriesInDomains(NSLibraryDirectory, NSUserDomainMask, YES) objectAtIndex:0]);
		if (Result.StartsWith(DocumentsBase) || Result.StartsWith(LibraryBase))
		{
			return Result;
		}
	}
#endif

	if (Result.StartsWith(TEXT("~/")))
	{
		static FString ReadPathBase = FString([[NSBundle mainBundle]bundlePath] );
		Result.ReplaceInline(TEXT("~"), TEXT(""));
		return ReadPathBase + Result;
	}
	
	FPaths::MakePlatformFilename(Result);
	
	Result.ReplaceInline(TEXT("../"), TEXT(""));
	Result.ReplaceInline(TEXT(".."), TEXT(""));
	Result.ReplaceInline(FPlatformProcess::BaseDir(), TEXT(""));

	for (FString AdditionalRootDirectory : FPlatformMisc::GetAdditionalRootDirectories())
	{
        AdditionalRootDirectory.ReplaceInline(TEXT("../"), TEXT(""));
        AdditionalRootDirectory.ReplaceInline(TEXT(".."), TEXT(""));
		if (Result.StartsWith(AdditionalRootDirectory) &&
			(Result.Len() == AdditionalRootDirectory.Len() || Result.Mid(AdditionalRootDirectory.Len()).StartsWith(FPlatformMisc::GetDefaultPathSeparator())))
		{
			static FString ReadPathBase = FString([NSSearchPathForDirectoriesInDomains(NSDocumentDirectory, NSUserDomainMask, YES) objectAtIndex:0]);

			// lowercase the second half of the path because ios
			Result = FPaths::Combine(ReadPathBase, Result.Mid(0,AdditionalRootDirectory.Len()), Result.Mid(AdditionalRootDirectory.Len()+1).ToLower());
			return Result;
        }
	}

	if(bForWrite)
	{
#if PLATFORM_TVOS
		// tvOS cannot write to the Documents directory. All files must be written to Library/Caches
		static FString PublicWritePathBase = FString([NSSearchPathForDirectoriesInDomains(NSCachesDirectory, NSUserDomainMask, YES) objectAtIndex:0]) + TEXT("/");

		return PublicWritePathBase + Result;
#else
		static FString PublicWritePathBase = FString([NSSearchPathForDirectoriesInDomains(NSDocumentDirectory, NSUserDomainMask, YES) objectAtIndex:0]) + TEXT("/");
		static FString PrivateWritePathBase = FString([NSSearchPathForDirectoriesInDomains(NSLibraryDirectory, NSUserDomainMask, YES) objectAtIndex:0]) + TEXT("/");
		
		return (bIsPublicWrite ? PublicWritePathBase : PrivateWritePathBase) + Result;
#endif
	}
	else
	{
		// if filehostip exists in the command line, cook on the fly read path should be used
		FString Value;
		// Cache this value as the command line doesn't change...
		static bool bHasHostIP = FParse::Value(FCommandLine::Get(), TEXT("filehostip"), Value) || FParse::Value(FCommandLine::Get(), TEXT("streaminghostip"), Value);
		static bool bIsIterative = FParse::Value(FCommandLine::Get(), TEXT("iterative"), Value);
		if (bHasHostIP)
		{
			static FString ReadPathBase = FString([NSSearchPathForDirectoriesInDomains(NSDocumentDirectory, NSUserDomainMask, YES) objectAtIndex:0]) + TEXT("/");
			return ReadPathBase + Result;
		}
		else if (bIsIterative)
		{
			static FString ReadPathBase = FString([NSSearchPathForDirectoriesInDomains(NSCachesDirectory, NSUserDomainMask, YES) objectAtIndex:0]) + TEXT("/");
			return ReadPathBase + Result.ToLower();
		}
		else
		{
			static FString ReadPathBase = FString([[NSBundle mainBundle] bundlePath]) + TEXT("/cookeddata/");
            return ReadPathBase + Result.ToLower();
		}
	}
}

int32 FIOSPlatformFile::OpenFileForMemMap(const FString& NormalizedFilename, EOpenMappedFlags OpenOptions, FString& OutFinalPath)
{
	int Flags = O_RDONLY;
	if (EnumHasAnyFlags(OpenOptions, EOpenMappedFlags::OpenForWrite))
	{
		Flags = O_RDWR;
		if (!EnumHasAnyFlags(OpenOptions, EOpenMappedFlags::OpenExisting))
		{
			Flags |= O_CREAT;
			Flags |= O_TRUNC;
		}
	}

	int32 Handle = -1;

	if (EnumHasAnyFlags(OpenOptions, EOpenMappedFlags::OpenForWrite))
	{
		FString FinalPath = ConvertToPlatformPath(NormalizedFilename, true, bCreatePublicFiles);
		Handle = open(TCHAR_TO_UTF8(*FinalPath), Flags, GetModeFlagsForOpenWrite());
		OutFinalPath = MoveTemp(FinalPath);
	}
	else
	{
		// check the read path
		FString FinalPath = ConvertToPlatformPath(NormalizedFilename, false, false);
		Handle = open(TCHAR_TO_UTF8(*FinalPath), Flags);
		if (Handle == -1)
		{
			// if not in the read path, check the private write path
			FinalPath = ConvertToPlatformPath(NormalizedFilename, true, false);
			Handle = open(TCHAR_TO_UTF8(*FinalPath), Flags);

			if (Handle == -1)
			{
				// if not in the private write path, check the public write path
				FinalPath = ConvertToPlatformPath(NormalizedFilename, true, true);
				Handle = open(TCHAR_TO_UTF8(*FinalPath), Flags);
			}
		}
		OutFinalPath = MoveTemp(FinalPath);
	}

	return Handle;
}

FString FIOSPlatformFile::GetSharedContainerPath(const TCHAR* AppGroupID)
{
	SCOPED_AUTORELEASE_POOL;
	
	NSString* GroupID = FString(AppGroupID).GetNSString();
	NSURL *ContainerUrl = [[NSFileManager defaultManager]
				  containerURLForSecurityApplicationGroupIdentifier:GroupID];
	
	if (ContainerUrl != nil)
	{
		return FString([ContainerUrl path]);
	}
	
	NSLog(@"[IOSPlatformFile] No SharedContainer with AppGroupID:%@", GroupID);
	return ""; 
}

/**
 * iOS platform file declaration
**/
IPlatformFile& IPlatformFile::GetPlatformPhysical()
{
	static FIOSPlatformFile IOSPlatformSingleton;
	//static FSandboxPlatformFile CookOnTheFlySandboxSingleton(false);
	//static FSandboxPlatformFile CookByTheBookSandboxSingleton(false);
	//static bool bInitialized = false;
	//if(!bInitialized)
	//{
	//	NSString *DocumentsPath = [NSSearchPathForDirectoriesInDomains(NSDocumentDirectory, NSUserDomainMask, YES) objectAtIndex:0];
	//	CookOnTheFlySandboxSingleton.Initialize(&IOSPlatformSingleton, ANSI_TO_TCHAR([DocumentsPath cStringUsingEncoding:NSASCIIStringEncoding]));

	//	DocumentsPath = [[NSBundle mainBundle] bundlePath];
	//	DocumentsPath = [DocumentsPath stringByAppendingPathComponent:@"/CookedContent/"];
	//	CookByTheBookSandboxSingleton.Initialize(&CookOnTheFlySandboxSingleton, ANSI_TO_TCHAR([DocumentsPath cStringUsingEncoding:NSASCIIStringEncoding]));

	//	bInitialized = true;
	//}

	//return CookByTheBookSandboxSingleton;

	return IOSPlatformSingleton;
}
