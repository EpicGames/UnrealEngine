// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	IOSPlatformMemory.cpp: IOS platform memory functions
=============================================================================*/

#include "IOS/IOSPlatformMemory.h"
#include "Misc/CoreStats.h"
#include "HAL/MallocBinned.h"
#include "HAL/MallocAnsi.h"
#include "GenericPlatform/GenericPlatformMemoryPoolStats.h"
#include "Misc/CoreDelegates.h"

void FIOSPlatformMemory::OnOutOfMemory(uint64 Size, uint32 Alignment)
{
	auto HandleOOM = [&]()
	{
		// Update memory stats before we enter the crash handler.
		OOMAllocationSize = Size;
		OOMAllocationAlignment = Alignment;
    
		bIsOOM = true;

		const int ErrorMsgSize = 256;
		TCHAR ErrorMsg[ErrorMsgSize];
		FPlatformMisc::GetSystemErrorMessage(ErrorMsg, ErrorMsgSize, 0);
    
		FPlatformMemoryStats PlatformMemoryStats = FPlatformMemory::GetStats();
		if (BackupOOMMemoryPool)
		{
			FPlatformMemory::BinnedFreeToOS(BackupOOMMemoryPool, FPlatformMemory::GetBackMemoryPoolSize());
			UE_LOGF(LogMemory, Warning, "Freeing %d bytes from backup pool to handle out of memory.", FPlatformMemory::GetBackMemoryPoolSize());
        
			LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelFree(ELLMTracker::Default, BackupOOMMemoryPool));
		}
    
		UE_LOG(LogMemory, Warning, TEXT("MemoryStats:")\
			   TEXT("\n\tAvailablePhysical %llu")\
			   TEXT("\n\t AvailableVirtual %llu")\
			   TEXT("\n\t     UsedPhysical %llu")\
			   TEXT("\n\t PeakUsedPhysical %llu")\
			   TEXT("\n\t      UsedVirtual %llu")\
			   TEXT("\n\t  PeakUsedVirtual %llu"),
			   (uint64)PlatformMemoryStats.AvailablePhysical,
			   (uint64)PlatformMemoryStats.AvailableVirtual,
			   (uint64)PlatformMemoryStats.UsedPhysical,
			   (uint64)PlatformMemoryStats.PeakUsedPhysical,
			   (uint64)PlatformMemoryStats.UsedVirtual,
			   (uint64)PlatformMemoryStats.PeakUsedVirtual);
    
		// let any registered handlers go
		FCoreDelegates::GetMemoryTrimDelegate().Broadcast();
    
		// ErrorMsg might be unrelated to OoM error in some cases as the code that calls OnOutOfMemory could have called other system functions that modified errno
		UE_LOGF(LogMemory, Warning, "Ran out of memory allocating %llu bytes with alignment %u. Last error msg: %ls.", Size, Alignment, ErrorMsg);
    
		// make this a fatal error that ends here not in the log
		UE_FORCE_CRASH();
	};
	
	UE_CALL_ONCE(HandleOOM);
	FPlatformProcess::SleepInfinite(); // Unreachable
}
