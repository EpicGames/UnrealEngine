// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	IOSPlatformMisc.mm: iOS implementations of misc functions
=============================================================================*/

#include "IOS/IOSPlatformMisc.h"

#include "Apple/AppleStringUtils.h"
#include "Apple/ApplePlatformMisc.h"
#include "Apple/ApplePlatformCrashContext.h"
#include "Delegates/Delegate.h"
#include "HAL/ExceptionHandling.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformOutputDevices.h"
#include "Internationalization/Culture.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Regex.h"
#include "IOSChunkInstaller.h"
#include "IOS/IOSMallocZone.h"
#include "IOS/IOSPlatformCrashContext.h"
#include "IOS/IOSPlatformPLCrashReporterIncludes.h"
#include "Misc/App.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreDelegates.h"
#include "Misc/EngineVersion.h"
#include "Misc/FeedbackContext.h"
#include "Misc/OutputDeviceError.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Misc/PathViews.h"
#include "Misc/ScopeExit.h"
#include "Misc/SecureHash.h"
#include "Modules/ModuleManager.h"
#include <atomic>
#include "Templates/Function.h"
#include "Templates/UniquePtr.h"
#include "Containers/StringConv.h"
#include "Containers/StringView.h"
#include "Apple/PreAppleSystemHeaders.h"
#include "IOS/IOSSystemIncludes.h"

#if !PLATFORM_TVOS && !PLATFORM_VISIONOS
#include <AdSupport/ASIdentifierManager.h>
#include "MarketplaceKit.h"
#endif // !PLATFORM_TVOS && !PLATFORM_VISIONOS

#import <DeviceCheck/DeviceCheck.h>
#import <Foundation/Foundation.h>
#import <mach-o/dyld.h>
#include <netinet/in.h>
#include <SystemConfiguration/SystemConfiguration.h>
#if UE_WITH_STORE_KIT
#import <StoreKit/StoreKit.h>
#endif

#import <UserNotifications/UserNotifications.h>

#include <sys/sysctl.h> // sysctlbyname

#include "Apple/PostAppleSystemHeaders.h"
#include "Apple/ScopeAutoreleasePool.h"

#if WITH_APPLICATION_CORE
#include "IOS/IOSApplication.h"
#include "IOS/IOSAppDelegate.h"
#include "IOS/IOSView.h"
#include "IOSConfigRules.h"
#endif

#include <Metal/Metal.h>

#if !defined ENABLE_ADVERTISING_IDENTIFIER
	#define ENABLE_ADVERTISING_IDENTIFIER 0
#endif

#include "Misc/AssertionMacros.h"

/** Amount of free memory in MB reported by the system at startup */
CORE_API int32 GStartupFreeMemoryMB;

extern CORE_API bool GIsGPUCrashed;

/** Global pointer to memory warning handler */
void (* GMemoryWarningHandler)(const FGenericMemoryWarningContext& Context) = NULL;

/** global for showing the splash screen */
bool GShowSplashScreen = true;

/** global for querying virtual keyboard input field height (without including keyboard itself) */
std::atomic<float> GIOSVirtualKeyboardInputFieldHeight{0.0f};

#if !UE_BUILD_SHIPPING
/** global for showing debug console */
bool GDebugConsoleOpen = false;
#endif

static int32 GetFreeMemoryMB()
{
	FPlatformMemoryStats MemoryStats = FPlatformMemory::GetStats();
	return MemoryStats.AvailablePhysical / 1024 / 1024;
}

void FIOSPlatformMisc::PlatformInit()
{
	// PlatformInit() starts the UI thread which creates the framebuffer and it requires
	// "r.MobileContentScaleFactor" and "r.Mobile.DesiredResX/Y" to be available before 
	// it's creation, so need to cache those value now.
#if WITH_APPLICATION_CORE
	[[IOSAppDelegate GetDelegate] LoadScreenResolutionModifiers];
		
	FAppEntry::PlatformInit();
#else
	check(false); // This code path is not implemented
#endif

	// Increase the maximum number of simultaneously open files
	struct rlimit Limit;
	Limit.rlim_cur = OPEN_MAX;
	Limit.rlim_max = RLIM_INFINITY;
	int32 Result = setrlimit(RLIMIT_NOFILE, &Limit);
	check(Result == 0);

	// Check for required entitlements
	TArray<FString> RequiredEntitlements;
	GConfig->GetArray(TEXT("/Script/IOSRuntimeSettings.IOSRuntimeSettings"), TEXT("RequiredEntitlements"), RequiredEntitlements, GEngineIni);
	for (const FString& Entitlement : RequiredEntitlements)
	{
		if (!FIOSPlatformMisc::IsEntitlementEnabled(TCHAR_TO_ANSI(*Entitlement)))
		{
#if UE_BUILD_SHIPPING
			// We don't want this to be counted as a legit crash, as this should only
			// happen if users are messing with the IPA.
			_Exit(0);
#else
			UE_LOGF(LogInit, Fatal, "App does not have required entitlement %ls.", *Entitlement);
#endif
		}
	}

	// Identity.
	UE_LOGF(LogInit, Log, "Computer: %ls", FPlatformProcess::ComputerName() );
	UE_LOGF(LogInit, Log, "User: %ls", FPlatformProcess::UserName() );

	
	const FPlatformMemoryConstants& MemoryConstants = FPlatformMemory::GetConstants();
	UE_LOGF(LogInit, Log, "CPU Page size=%zu, Cores=%i", MemoryConstants.PageSize, FPlatformMisc::NumberOfCores() );

	// Timer resolution.
	UE_LOGF(LogInit, Log, "High frequency timer resolution =%f MHz", 0.000001 / FPlatformTime::GetSecondsPerCycle() );
	GStartupFreeMemoryMB = GetFreeMemoryMB();
	UE_LOGF(LogInit, Log, "Free Memory at startup: %d MB", GStartupFreeMemoryMB);

	// create the Documents/<GameName>/Content directory so we can exclude it from iCloud backup
	FString ResultStr = FPaths::ProjectContentDir();
	ResultStr.ReplaceInline(TEXT("../"), TEXT(""));
	ResultStr.ReplaceInline(TEXT(".."), TEXT(""));
	ResultStr.ReplaceInline(FPlatformProcess::BaseDir(), TEXT(""));
#if FILESHARING_ENABLED
	FString DownloadPath = FString([NSSearchPathForDirectoriesInDomains(NSLibraryDirectory, NSUserDomainMask, YES) objectAtIndex:0]) + TEXT("/");
#else
	FString DownloadPath = FString([NSSearchPathForDirectoriesInDomains(NSDocumentDirectory, NSUserDomainMask, YES) objectAtIndex:0]) + TEXT("/");
#endif
	ResultStr = DownloadPath + ResultStr;
	NSURL* URL = [NSURL fileURLWithPath : ResultStr.GetNSString()];
	if (![[NSFileManager defaultManager] fileExistsAtPath:[URL path]])
	{
		[[NSFileManager defaultManager] createDirectoryAtURL:URL withIntermediateDirectories : YES attributes : nil error : nil];
	}

	// mark it to not be uploaded
	NSError *error = nil;
	BOOL success = [URL setResourceValue : [NSNumber numberWithBool : YES] forKey : NSURLIsExcludedFromBackupKey error : &error];
	if (!success)
	{
		NSLog(@"Error excluding %@ from backup %@",[URL lastPathComponent], error);
	}

	// create the Documents/Engine/Content directory so we can exclude it from iCloud backup
	ResultStr = FPaths::EngineContentDir();
	ResultStr.ReplaceInline(TEXT("../"), TEXT(""));
	ResultStr.ReplaceInline(TEXT(".."), TEXT(""));
	ResultStr.ReplaceInline(FPlatformProcess::BaseDir(), TEXT(""));
	ResultStr = DownloadPath + ResultStr;
	URL = [NSURL fileURLWithPath : ResultStr.GetNSString()];
	if (![[NSFileManager defaultManager] fileExistsAtPath:[URL path]])
	{
		[[NSFileManager defaultManager] createDirectoryAtURL:URL withIntermediateDirectories : YES attributes : nil error : nil];
	}

	// mark it to not be uploaded
	success = [URL setResourceValue : [NSNumber numberWithBool : YES] forKey : NSURLIsExcludedFromBackupKey error : &error];
	if (!success)
	{
		NSLog(@"Error excluding %@ from backup %@",[URL lastPathComponent], error);
	}
	
	FApplePlatformMisc::PlatformInit();
}

void FIOSPlatformMisc::PlatformTearDown()
{
	FApplePlatformMisc::PlatformTearDown();
}

// Defines the PlatformFeatures module name for iOS, used by PlatformFeatures.h.
const TCHAR* FIOSPlatformMisc::GetPlatformFeaturesModuleName()
{
	return TEXT("IOSPlatformFeatures");
}

void FIOSPlatformMisc::PlatformHandleSplashScreen(bool ShowSplashScreen)
{
    if (GShowSplashScreen != ShowSplashScreen)
    {
        // put a render thread job to turn off the splash screen after the first render flip
        FGraphEventRef SplashTask = FFunctionGraphTask::CreateAndDispatchWhenReady([ShowSplashScreen]()
        {
            GShowSplashScreen = ShowSplashScreen;
        }, TStatId(), NULL, ENamedThreads::ActualRenderingThread);
    }
}

bool FIOSPlatformMisc::AllowThreadHeartBeat()
{
	static uint32 AllowThreadHeartBeatOnce = -1;
	if (AllowThreadHeartBeatOnce == -1)
	{
		const FString* AllowThreadHeartBeatConfigVar = GetConfigRulesVariable(TEXT("EnableThreadHeartBeat"));
		if (AllowThreadHeartBeatConfigVar)
		{
			AllowThreadHeartBeatOnce = (uint32)AllowThreadHeartBeatConfigVar->Equals("true", ESearchCase::IgnoreCase);
		}
		else
		{
			AllowThreadHeartBeatOnce = 0;
		}
	}

	return AllowThreadHeartBeatOnce == 1;
}

const TCHAR* FIOSPlatformMisc::GamePersistentDownloadDir()
{
    static FString GamePersistentDownloadDir;
    
    if (GamePersistentDownloadDir.Len() == 0)
    {
        FString BaseProjectDir = ProjectDir();
        
        if (BaseProjectDir.Len() > 0)
        {
            GamePersistentDownloadDir = BaseProjectDir / TEXT("PersistentDownloadDir");
        }
        
        // create the directory so we can exclude it from iCloud backup
        FString Result = GamePersistentDownloadDir;
        Result.ReplaceInline(TEXT("../"), TEXT(""));
        Result.ReplaceInline(TEXT(".."), TEXT(""));
        Result.ReplaceInline(FPlatformProcess::BaseDir(), TEXT(""));
#if FILESHARING_ENABLED
        FString DownloadPath = FString([NSSearchPathForDirectoriesInDomains(NSLibraryDirectory, NSUserDomainMask, YES) objectAtIndex:0]) + TEXT("/");
#else
		FString DownloadPath = FString([NSSearchPathForDirectoriesInDomains(NSDocumentDirectory, NSUserDomainMask, YES) objectAtIndex:0]) + TEXT("/");
#endif
        Result = DownloadPath + Result;
        NSURL* URL = [NSURL fileURLWithPath : Result.GetNSString()];
#if !PLATFORM_TVOS		
		// this folder is expected to not exist on TVOS 
        if (![[NSFileManager defaultManager] fileExistsAtPath:[URL path]])
        {
            [[NSFileManager defaultManager] createDirectoryAtURL:URL withIntermediateDirectories : YES attributes : nil error : nil];
        }
        
        // mark it to not be uploaded
        NSError *error = nil;
        BOOL success = [URL setResourceValue : [NSNumber numberWithBool : YES] forKey : NSURLIsExcludedFromBackupKey error : &error];
        if (!success)
        {
            NSLog(@"Error excluding %@ from backup %@",[URL lastPathComponent], error);
        }
#endif		
    }
    return *GamePersistentDownloadDir;
}

EAppReturnType::Type FIOSPlatformMisc::MessageBoxExt( EAppMsgType::Type MsgType, const TCHAR* Text, const TCHAR* Caption )
{
#if WITH_APPLICATION_CORE
	extern EAppReturnType::Type MessageBoxExtImpl( EAppMsgType::Type MsgType, const TCHAR* Text, const TCHAR* Caption );
	return MessageBoxExtImpl(MsgType, Text, Caption);
#else
	return {};
#endif
}

int FIOSPlatformMisc::GetAudioVolume()
{
#if WITH_APPLICATION_CORE
	return [[IOSAppDelegate GetDelegate] GetAudioVolume];
#else
	return 0;
#endif
}

int32 FIOSPlatformMisc::GetDeviceVolume()
{
#if WITH_APPLICATION_CORE
	return [[IOSAppDelegate GetDelegate] GetAudioVolume];
#else
	return 0;
#endif
}

bool FIOSPlatformMisc::AreHeadphonesPluggedIn()
{
#if WITH_APPLICATION_CORE
	return [[IOSAppDelegate GetDelegate] AreHeadphonesPluggedIn];
#else
	return false;
#endif
}

int FIOSPlatformMisc::GetBatteryLevel()
{
#if WITH_APPLICATION_CORE
	return [[IOSAppDelegate GetDelegate] GetBatteryLevel];
#else
	return 0;
#endif
}

float FIOSPlatformMisc::GetBrightness()
{
#if !PLATFORM_TVOS && !PLATFORM_VISIONOS && WITH_APPLICATION_CORE
	return (float)[[[IOSAppDelegate GetDelegate] window] screen].brightness;
#else
	return 1.0f;
#endif // !PLATFORM_TVOS && !PLATFORM_VISIONOS
}

void FIOSPlatformMisc::SetBrightness(float Brightness)
{
#if !PLATFORM_TVOS && !PLATFORM_VISIONOS && WITH_APPLICATION_CORE
	dispatch_async(dispatch_get_main_queue(), ^{
		[[[IOSAppDelegate GetDelegate] window] screen].brightness = Brightness;
	});
#endif // !PLATFORM_TVOS && !PLATFORM_VISIONOS
}

bool FIOSPlatformMisc::IsRunningOnBattery()
{
#if WITH_APPLICATION_CORE
	return [[IOSAppDelegate GetDelegate] IsRunningOnBattery];
#else
	return false;
#endif
}

float FIOSPlatformMisc::GetDeviceTemperatureLevel()
{
#if WITH_APPLICATION_CORE && !PLATFORM_TVOS
    switch ([[IOSAppDelegate GetDelegate] GetThermalState])
    {
        case NSProcessInfoThermalStateNominal:	return (float)FCoreDelegates::ETemperatureSeverity::Good;
        case NSProcessInfoThermalStateFair:		return (float)FCoreDelegates::ETemperatureSeverity::Bad;
        case NSProcessInfoThermalStateSerious:	return (float)FCoreDelegates::ETemperatureSeverity::Serious;
        case NSProcessInfoThermalStateCritical:	return (float)FCoreDelegates::ETemperatureSeverity::Critical;
    }
#endif
	return -1.0f;
}

EDeviceThermalState FIOSPlatformMisc::GetDeviceThermalState()
{
#if WITH_APPLICATION_CORE && !PLATFORM_TVOS
	// We are mapping the ios thermal state values onto the first four of our thermal state enum values, which are based on 
	// the android thermal status values.
	// Nominal and None both refer to states where the device is entirely fine on heat and not throttling.
	// Fair/Serious Light/Moderate are less exactly comparable.  Android, per docs, does some throttling.  IOS does not.  But
	// in both cases the device is physically getting warmer.
	// Critical/Severe indicate that serious throttling is kicking in and the device is probably noticably hot to touch.
	switch ([[IOSAppDelegate GetDelegate]GetThermalState] )
	{
	case NSProcessInfoThermalStateNominal:	return EDeviceThermalState::None;
	case NSProcessInfoThermalStateFair:		return EDeviceThermalState::Light;
	case NSProcessInfoThermalStateSerious:	return EDeviceThermalState::Moderate;
	case NSProcessInfoThermalStateCritical:	return EDeviceThermalState::Severe;
	}
#endif
	return EDeviceThermalState::Unsupported;
}

bool FIOSPlatformMisc::IsInLowPowerMode()
{
#if !PLATFORM_TVOS
    bool bInLowPowerMode = [[NSProcessInfo processInfo] isLowPowerModeEnabled];
    return bInLowPowerMode;
#else
    return false;
#endif
}

bool FIOSPlatformMisc::IsDesignedForIpadOnVisionOS()
{
	static BOOL bIsVision;
	static dispatch_once_t onceToken;
	dispatch_once(&onceToken, ^{
		bIsVision = (NSClassFromString(@"UIWindowSceneGeometryPreferencesVision") != nil);
	});
	return bIsVision;
}

bool FIOSPlatformMisc:: IsDesignedForIpadOnMacOS()
{
	static BOOL bIsMac;
	static dispatch_once_t onceToken;
	dispatch_once(&onceToken, ^{
		bIsMac = (NSClassFromString(@"UIWindowSceneGeometryPreferencesMac") != nil);
	});
	return bIsMac;
}


#if !PLATFORM_TVOS && !PLATFORM_VISIONOS
UIInterfaceOrientationMask GetUIInterfaceOrientationMask(EDeviceScreenOrientation ScreenOrientation)
{
	switch (ScreenOrientation)
	{
	default:
		// Fallthrough...
	case EDeviceScreenOrientation::Unknown:
		return UIInterfaceOrientationMaskAll;
	case EDeviceScreenOrientation::Portrait:
		return UIInterfaceOrientationMaskPortrait;
	case EDeviceScreenOrientation::PortraitUpsideDown:
		return UIInterfaceOrientationMaskPortraitUpsideDown;
	case EDeviceScreenOrientation::LandscapeLeft:
		return UIInterfaceOrientationMaskLandscapeLeft;
	case EDeviceScreenOrientation::LandscapeRight:
		return UIInterfaceOrientationMaskLandscapeRight;
	case EDeviceScreenOrientation::FaceUp:
		return UIInterfaceOrientationMaskAll;
	case EDeviceScreenOrientation::FaceDown:
		return UIInterfaceOrientationMaskAll;
	case EDeviceScreenOrientation::PortraitSensor:
		return UIInterfaceOrientationMaskPortrait;
	case EDeviceScreenOrientation::LandscapeSensor:
		return UIInterfaceOrientationMaskLandscape;
	case EDeviceScreenOrientation::FullSensor:
		return UIInterfaceOrientationMaskAll;
	}
}
#endif

EDeviceScreenOrientation FIOSPlatformMisc::GetDeviceOrientation()
{
#if WITH_APPLICATION_CORE && !PLATFORM_TVOS && !PLATFORM_VISIONOS
	IOSAppDelegate* AppDelegate = [IOSAppDelegate GetDelegate];
	if (AppDelegate.InterfaceOrientation == UIInterfaceOrientationUnknown)
	{
		AppDelegate.InterfaceOrientation = [[[[[UIApplication sharedApplication] delegate] window] windowScene] interfaceOrientation];
	}

	return [IOSAppDelegate ConvertFromUIInterfaceOrientation:AppDelegate.InterfaceOrientation];
#else
	return EDeviceScreenOrientation::Unknown;
#endif
}

void FIOSPlatformMisc::SetAllowedDeviceOrientation(EDeviceScreenOrientation NewAllowedDeviceOrientation)
{
	AllowedDeviceOrientation = NewAllowedDeviceOrientation;
	
#if WITH_APPLICATION_CORE && !PLATFORM_TVOS && !PLATFORM_VISIONOS
	IOSAppDelegate* AppDelegate = [IOSAppDelegate GetDelegate];
	AppDelegate.IOSView->SupportedInterfaceOrientations = GetUIInterfaceOrientationMask(NewAllowedDeviceOrientation);

#if IOS_ROTATION_DEBUG_LOGGING
	FPlatformMisc::LowLevelOutputDebugStringf(TEXT("[rotation] SetAllowedDeviceOrientation mask %d\n"), (int)GetUIInterfaceOrientationMask(NewAllowedDeviceOrientation));
#endif

	// Check rotation state
	[AppDelegate didRotate:nil];

	// Attempt to rotate to desired orientation
	[IOSAppDelegate UpdateSupportedInterfaceOrientations];
#endif
}

bool FIOSPlatformMisc::HasPlatformFeature(const TCHAR* FeatureName)
{
	if (FCString::Stricmp(FeatureName, TEXT("Metal")) == 0)
	{
		return true;
	}

	return FGenericPlatformMisc::HasPlatformFeature(FeatureName);
}

FString GetIOSDeviceIDString()
{
	static FString CachedResult;
	static bool bCached = false;
	if (!bCached)
	{
		// get the device hardware type string length
		size_t DeviceIDLen;
		sysctlbyname("hw.machine", NULL, &DeviceIDLen, NULL, 0);

		// get the device hardware type
		char* DeviceID = (char*)malloc(DeviceIDLen);
		sysctlbyname("hw.machine", DeviceID, &DeviceIDLen, NULL, 0);

		CachedResult = ANSI_TO_TCHAR(DeviceID);
		bCached = true;

		free(DeviceID);
		
		// arm simulator
		// @todo test intel simulator
		if (CachedResult == "arm64")
		{
#if PLATFORM_VISIONOS
			CachedResult = TEXT("VisionPro0,1");
#elif PLATFORM_TVOS
			CachedResult = TEXT("AppleTV0,1");
#else
			NSString* ModelID = [[NSProcessInfo processInfo] environment][@"SIMULATOR_MODEL_IDENTIFIER"];
			CachedResult = FString(ModelID);
#endif
		}
	}

	return CachedResult;
}

static void MatchIOSDeviceMapping(const FString& DeviceIDString, const TCHAR* DeviceMappingSection, FString& IOSDeviceProfileName)
{
	TArray<FString> Mappings;
	if (ensure(GConfig->GetSection(DeviceMappingSection, Mappings, GDeviceProfilesIni)))
	{
		for (const FString& MappingString : Mappings)
		{
			FString MappingRegex, ProfileName;
			if (MappingString.Split(TEXT("="), &MappingRegex, &ProfileName))
			{
				const FRegexPattern RegexPattern(MappingRegex);
				FRegexMatcher RegexMatcher(RegexPattern, *DeviceIDString);
				if (RegexMatcher.FindNext())
				{
					FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Matched %s as %s") LINE_TERMINATOR, *MappingRegex, *ProfileName);
					IOSDeviceProfileName = ProfileName;
					break;
				}
			}
			else
			{
				FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Invalid %s: %s") LINE_TERMINATOR, DeviceMappingSection, *MappingString);
			}
		}
	}
}

const TCHAR* FIOSPlatformMisc::GetDefaultDeviceProfileName()
{
	static FString IOSDeviceProfileName;
	if (IOSDeviceProfileName.Len() == 0)
	{		
		FString DeviceIDString = GetIOSDeviceIDString();
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Device Type: %s") LINE_TERMINATOR, *DeviceIDString);
		
		MatchIOSDeviceMapping( DeviceIDString, TEXT("IOSDeviceMappings"), IOSDeviceProfileName );
		if (IOSDeviceProfileName.Len() == 0)
		{
			MatchIOSDeviceMapping( DeviceIDString, TEXT("IOSFallbackDeviceMappings"), IOSDeviceProfileName );	
		}
		check(IOSDeviceProfileName.Len() > 0)
	}

	return *IOSDeviceProfileName;
}

// Deprecated in 4.26.
FIOSPlatformMisc::EIOSDevice FIOSPlatformMisc::GetIOSDeviceType()
{
	// default to unknown
	static EIOSDevice DeviceType = IOS_Unknown;

	// if we've already figured it out, return it
	if (DeviceType != IOS_Unknown)
	{
		return DeviceType;
	}

	const FString DeviceIDString = GetIOSDeviceIDString();

    FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Device Type: %s") LINE_TERMINATOR, *DeviceIDString);

#if PLATFORM_VISIONOS
	DeviceType = IOS_RealityPro;
#else
	
    // iPods
	if (DeviceIDString.StartsWith(TEXT("iPod")))
	{
		// get major revision number
        int Major = FCString::Atoi(&DeviceIDString[4]);

		if (Major == 5)
		{
			DeviceType = IOS_IPodTouch5;
		}
		else if (Major == 7)
		{
			DeviceType = IOS_IPodTouch6;
		}
		else if (Major >= 9)
		{
			DeviceType = IOS_IPodTouch7;
		}
	}
	// iPads
	else if (DeviceIDString.StartsWith(TEXT("iPad")))
	{
		// get major revision number
		const int Major = FCString::Atoi(&DeviceIDString[4]);
		const int CommaIndex = DeviceIDString.Find(TEXT(","), ESearchCase::CaseSensitive, ESearchDir::FromStart, 4);
		const int Minor = FCString::Atoi(&DeviceIDString[CommaIndex + 1]);

		// iPad2,[1|2|3] is iPad 2 (1 - wifi, 2 - gsm, 3 - cdma)
		if (Major == 2)
		{
			// iPad2,5+ is the new iPadMini, anything higher will use these settings until released
			if (Minor >= 5)
			{
				DeviceType = IOS_IPadMini;
			}
			else
			{
				DeviceType = IOS_IPad2;
			}
		}
		// iPad3,[1|2|3] is iPad 3 and iPad3,4+ is iPad (4th generation)
		else if (Major == 3)
		{
			if (Minor <= 3)
			{
				DeviceType = IOS_IPad3;
			}
			// iPad3,4+ is the new iPad, anything higher will use these settings until released
			else if (Minor >= 4)
			{
				DeviceType = IOS_IPad4;
			}
		}
		// iPadAir and iPad Mini 2nd Generation
		else if (Major == 4)
		{
			if (Minor >= 4)
			{
				DeviceType = IOS_IPadMini2;
			}
			else
			{
				DeviceType = IOS_IPadAir;
			}
		}
		// iPad Air 2 and iPadMini 4
		else if (Major == 5)
		{
			if (Minor == 1 || Minor == 2)
			{
				DeviceType = IOS_IPadMini4;
			}
			else
			{
				DeviceType = IOS_IPadAir2;
			}
		}
		else if (Major == 6)
		{
			if (Minor == 3 || Minor == 4)
			{
				DeviceType = IOS_IPadPro_97;
			}
			else if (Minor == 11 || Minor == 12)
			{
				DeviceType = IOS_IPad5;
			}
			else
			{
				DeviceType = IOS_IPadPro_129;
			}
		}
		else if (Major == 7)
		{
			if (Minor == 3 || Minor == 4)
			{
				DeviceType = IOS_IPadPro_105;
			}
			else if (Minor == 5 || Minor == 6)
			{
				DeviceType = IOS_IPad6;
			}
			else if (Minor == 11 || Minor == 12)
			{
				DeviceType = IOS_IPad7;
			}
			else
			{
				DeviceType = IOS_IPadPro2_129;
			}
		}
		else if (Major == 8)
		{
			if (Minor <= 4)
			{
				DeviceType = IOS_IPadPro_11;
			}
			else if (Minor <= 8)
			{
				DeviceType = IOS_IPadPro3_129;
			}
			else if (Minor <= 10)
			{
				DeviceType = IOS_IPadPro2_11;
			}
			else
			{
				DeviceType = IOS_IPadPro4_129;
			}
		}
        else if (Major == 11)
        {
            if (Minor <= 2)
            {
                DeviceType = IOS_IPadMini5;
            }
            else
            {
                DeviceType = IOS_IPadAir3;
            }
        }
		// Default to highest settings currently available for any future device
		else if (Major >= 9)
		{
			DeviceType = IOS_IPadPro4_129;
		}
	}
	// iPhones
	else if (DeviceIDString.StartsWith(TEXT("iPhone")))
	{
        const int Major = FCString::Atoi(&DeviceIDString[6]);
		const int CommaIndex = DeviceIDString.Find(TEXT(","), ESearchCase::CaseSensitive, ESearchDir::FromStart, 6);
		const int Minor = FCString::Atoi(&DeviceIDString[CommaIndex + 1]);

		if (Major == 3)
		{
			DeviceType = IOS_IPhone4;
		}
		else if (Major == 4)
		{
			DeviceType = IOS_IPhone4S;
		}
		else if (Major == 5)
		{
			DeviceType = IOS_IPhone5;
		}
		else if (Major == 6)
		{
			DeviceType = IOS_IPhone5S;
		}
		else if (Major == 7)
		{
			if (Minor == 1)
			{
				DeviceType = IOS_IPhone6Plus;
			}
			else if (Minor == 2)
			{
				DeviceType = IOS_IPhone6;
			}
		}
		else if (Major == 8)
		{
			// note that Apple switched the minor order around between 6 and 6S (gotta keep us on our toes!)
			if (Minor == 1)
			{
				DeviceType = IOS_IPhone6S;
			}
			else if (Minor == 2)
			{
				DeviceType = IOS_IPhone6SPlus;
			}
			else if (Minor == 4)
			{
				DeviceType = IOS_IPhoneSE;
			}
		}
		else if (Major == 9)
		{
            if (Minor == 1 || Minor == 3)
            {
                DeviceType = IOS_IPhone7;
            }
            else if (Minor == 2 || Minor == 4)
            {
                DeviceType = IOS_IPhone7Plus;
            }
		}
        else if (Major == 10)
        {
			if (Minor == 1 || Minor == 4)
			{
				DeviceType = IOS_IPhone8;
			}
			else if (Minor == 2 || Minor == 5)
			{
				DeviceType = IOS_IPhone8Plus;
			}
			else if (Minor == 3 || Minor == 6)
			{
				DeviceType = IOS_IPhoneX;
			}
		}
        else if (Major == 11)
        {
            if (Minor == 2)
            {
                DeviceType = IOS_IPhoneXS;
            }
            else if (Minor == 4 || Minor == 6)
            {
                DeviceType = IOS_IPhoneXSMax;
            }
            else if (Minor == 8)
            {
                DeviceType = IOS_IPhoneXR;
            }
        }
		else if (Major == 12)
		{
			if (Minor < 3)
			{
				DeviceType = IOS_IPhone11;
			}
			else if (Minor < 5)
			{
				DeviceType = IOS_IPhone11Pro;
			}
			else if (Minor < 7)
			{
				DeviceType = IOS_IPhone11ProMax;
			}
			else if (Minor == 8)
			{
				DeviceType = IOS_IPhoneSE2;
			}
		}
		else if (Major >= 13)
		{
			// for going forward into unknown devices (like 8/8+?), we can't use Minor,
			// so treat devices with a scale > 2.5 to be 6SPlus type devices, < 2.5 to be 6S type devices
			if ([UIScreen mainScreen].scale > 2.5f)
			{
				DeviceType = IOS_IPhone11ProMax;
			}
			else
			{
				DeviceType = IOS_IPhone11Pro;
			}
		}
	}
	// tvOS
	else if (DeviceIDString.StartsWith(TEXT("AppleTV")))
	{
		const int Major = FCString::Atoi(&DeviceIDString[7]);
		const int CommaIndex = DeviceIDString.Find(TEXT(","), ESearchCase::CaseSensitive, ESearchDir::FromStart, 6);
		const int Minor = FCString::Atoi(&DeviceIDString[CommaIndex + 1]);

		if (Major == 5)
		{
			DeviceType = IOS_AppleTV;
		}
		else if (Major == 6)
		{
			DeviceType = IOS_AppleTV4K;
		}
		else if (Major >= 6)
		{
			DeviceType = IOS_AppleTV4K;
		}
	}
	// simulator
	else if (DeviceIDString.StartsWith(TEXT("x86")))
	{
		// iphone
		if (UI_USER_INTERFACE_IDIOM() == UIUserInterfaceIdiomPhone)
		{
			CGSize result = [[UIScreen mainScreen] bounds].size;
			if(result.height >= 586)
			{
				DeviceType = IOS_IPhone5;
			}
			else
			{
				DeviceType = IOS_IPhone4S;
			}
		}
		else
		{
			if ([[UIScreen mainScreen] scale] > 1.0f)
			{
				DeviceType = IOS_IPad4;
			}
			else
			{
				DeviceType = IOS_IPad2;
			}
		}
	}
#endif
	
	// if this is unknown at this point, we have a problem
	if (DeviceType == IOS_Unknown)
	{
		UE_LOGF(LogInit, Fatal, "This IOS device type is not supported by UE4 [%ls]\n", *FString(DeviceIDString));
	}

	return DeviceType;
}

int FIOSPlatformMisc::GetDefaultStackSize()
{
	return 512 * 1024;
}

void FIOSPlatformMisc::SetMemoryWarningHandler(void (* InHandler)(const FGenericMemoryWarningContext& Context))
{
	GMemoryWarningHandler = InHandler;
}

bool FIOSPlatformMisc::HasMemoryWarningHandler()
{
	return GMemoryWarningHandler != nullptr;
}

void FIOSPlatformMisc::HandleLowMemoryWarning()
{
#if UE_BUILD_SHIPPING || UE_BUILD_TEST
	static double LastPrintTime = 0;
	const double CurrentTime = FPlatformTime::Seconds();
	const double CheckInterval = 0.05; // Wait at least 50ms before re-printing information

	if (CurrentTime >= LastPrintTime + CheckInterval)
	{
		LastPrintTime = CurrentTime;
#endif

	UE_LOGF(LogInit, Log, "Low Memory Warning Triggered");
	UE_LOGF(LogInit, Log, "Free Memory at Startup: %d MB", GStartupFreeMemoryMB);
	UE_LOGF(LogInit, Log, "Free Memory Now       : %d MB", GetFreeMemoryMB());

#if UE_BUILD_SHIPPING || UE_BUILD_TEST
	}
#endif

	if(GMemoryWarningHandler != NULL)
	{
		FGenericMemoryWarningContext Context;
		GMemoryWarningHandler(Context);
	}
}

bool FIOSPlatformMisc::IsPackagedForDistribution()
{
#if !UE_BUILD_SHIPPING
	static bool PackagingModeCmdLine = FParse::Param(FCommandLine::Get(), TEXT("PACKAGED_FOR_DISTRIBUTION"));
	if (PackagingModeCmdLine)
	{
		return true;
	}
#endif
	NSString* PackagingMode = [[[NSBundle mainBundle] infoDictionary] objectForKey:@"EpicPackagingMode"];
	return PackagingMode != nil && [PackagingMode isEqualToString : @"Distribution"];
}

FString FIOSPlatformMisc::GetDeviceId()
{
#if GET_DEVICE_ID_UNAVAILABLE
	return FString();
#else
	// Check to see if this OS has this function

	if ([[UIDevice currentDevice] respondsToSelector:@selector(identifierForVendor)])
	{
	    NSUUID* Id = [[UIDevice currentDevice] identifierForVendor];
	    if (Id != nil)
	    {
		    NSString* IdfvString = [Id UUIDString];
		    FString IDFV(IdfvString);
		    return IDFV;
	    }
	}
	return FString();
#endif
}

FString FIOSPlatformMisc::GetOSVersion()
{
	return FString([[UIDevice currentDevice] systemVersion]);
}

bool FIOSPlatformMisc::GetDiskTotalAndFreeSpace(const FString& InPath, uint64& OutTotalNumberOfBytes, uint64& OutNumberOfFreeBytes, bool bInIncludeFreeable)
{
    SCOPED_AUTORELEASE_POOL;

    //On iOS 11 use new method to return disk space available for important usages
#if !PLATFORM_TVOS
    bool GetValueSuccess = false;
    
	// NSURLVolumeAvailableCapacityForImportantUsageKey
	// This key is more reliable because of how much free space can be "on demand" on iOS - on devices we've seen as much as 100gb
	// difference between the "ImportantUsage" and "Capacity" keys. Capacity is almost certainly not what you want. However, because "ImportantUsage" involves
	// accumulating all the data that could be freed if needed, it can be very slow (upwards of 100ms).
    NSNumber *FreeBytes = nil;
    NSURL *URL = [NSURL fileURLWithPath : NSHomeDirectory()];
    GetValueSuccess = [URL getResourceValue : &FreeBytes forKey : (bInIncludeFreeable ? NSURLVolumeAvailableCapacityForImportantUsageKey : NSURLVolumeAvailableCapacityKey) error : nil];
    if (FreeBytes)
    {
        OutNumberOfFreeBytes = [FreeBytes longLongValue];
    }
    
    NSNumber *TotalBytes = nil;
    GetValueSuccess = GetValueSuccess &&[URL getResourceValue : &TotalBytes forKey : NSURLVolumeTotalCapacityKey error : nil];
    if (TotalBytes)
    {
        OutTotalNumberOfBytes = [TotalBytes longLongValue];
    }
    
    if (GetValueSuccess
        && (OutNumberOfFreeBytes > 0)
        && (OutTotalNumberOfBytes > 0))
    {
        return true;
    }
#endif

    //fallback to old method if we didn't return above
    {
        NSDictionary<NSFileAttributeKey, id>* FSStat = [[NSFileManager defaultManager] attributesOfFileSystemForPath:NSHomeDirectory() error : nil];
        if (FSStat)
        {
            OutNumberOfFreeBytes = [[FSStat objectForKey : NSFileSystemFreeSize] longLongValue];
            OutTotalNumberOfBytes = [[FSStat objectForKey : NSFileSystemSize] longLongValue];

            return true;
        }

		// This should almost never happen to make sure to leave a breadcrumb if it does since
		// callers are probably not checking results.
		UE_LOGF(LogInit, Warning, "Call to GetDiskTotalAndFreeSpace failed.");
        return false;
    }
}


void FIOSPlatformMisc::RequestStoreReview()
{
#if UE_WITH_STORE_KIT && !PLATFORM_TVOS && !PLATFORM_VISIONOS
	if (FMarketplaceKitModule::GetCurrentTypeStatic() == EMarketplaceType::AppStore)
	{
		// Deprecated for iOS18.0 & visionOS 2.0.  To be replaced with Swift only StoreKit::RequestReviewAction call (UE-228925)
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		[SKStoreReviewController requestReviewInScene:[[[[UIApplication sharedApplication] delegate] window] windowScene]];
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
#endif
}

bool FIOSPlatformMisc::IsUpdateAvailable()
{
#if WITH_APPLICATION_CORE 
	return [[IOSAppDelegate GetDelegate] IsUpdateAvailable];
#else
	return false;
#endif
}

/**
* Returns a unique string for advertising identification
*
* @return the unique string generated by this platform for this device
*/
FString FIOSPlatformMisc::GetUniqueAdvertisingId()
{
#if WITH_APPLICATION_CORE && !PLATFORM_TVOS && !PLATFORM_VISIONOS && ENABLE_ADVERTISING_IDENTIFIER
	// Check to see if this OS has this function
	if ([[ASIdentifierManager sharedManager] respondsToSelector:@selector(advertisingIdentifier)])
	{
		NSString* IdfaString = [[[ASIdentifierManager sharedManager] advertisingIdentifier] UUIDString];
		FString IDFA(IdfaString);
		return IDFA;
	}
#endif
	return FString();
}

class IPlatformChunkInstall* FIOSPlatformMisc::GetPlatformChunkInstall()
{
	static IPlatformChunkInstall* ChunkInstall = nullptr;
	static bool bIniChecked = false;
	if (!ChunkInstall || !bIniChecked)
	{
		FString ProviderName;
		IPlatformChunkInstallModule* PlatformChunkInstallModule = nullptr;
		if (!GEngineIni.IsEmpty())
		{
			FString InstallModule;
			GConfig->GetString(TEXT("StreamingInstall"), TEXT("DefaultProviderName"), InstallModule, GEngineIni);
			FModuleStatus Status;
			if (FModuleManager::Get().QueryModule(*InstallModule, Status))
			{
				PlatformChunkInstallModule = FModuleManager::LoadModulePtr<IPlatformChunkInstallModule>(*InstallModule);
				if (PlatformChunkInstallModule != nullptr)
				{
					// Attempt to grab the platform installer
					ChunkInstall = PlatformChunkInstallModule->GetPlatformChunkInstall();
				}
			}
			else if (ProviderName == TEXT("IOSChunkInstaller"))
			{
				static FIOSChunkInstall Singleton;
				ChunkInstall = &Singleton;
			}
			bIniChecked = true;
		}
		if (!ChunkInstall)
		{
			// Placeholder instance
			ChunkInstall = FGenericPlatformMisc::GetPlatformChunkInstall();
		}
	}

	return ChunkInstall;
}

bool FIOSPlatformMisc::SupportsForceTouchInput()
{
#if WITH_APPLICATION_CORE && !PLATFORM_TVOS && !PLATFORM_VISIONOS
	return [[[IOSAppDelegate GetDelegate].IOSView traitCollection] forceTouchCapability];
#else
	return false;
#endif
}

#if !PLATFORM_TVOS && !PLATFORM_VISIONOS
static UIFeedbackGenerator* GFeedbackGenerator = nullptr;
#endif // !PLATFORM_TVOS
static EMobileHapticsType GHapticsType;
void FIOSPlatformMisc::PrepareMobileHaptics(EMobileHapticsType Type)
{
	// these functions must run on the main IOS thread
	dispatch_async(dispatch_get_main_queue(), ^
	{
#if !PLATFORM_TVOS && !PLATFORM_VISIONOS
		if (GFeedbackGenerator != nullptr)
		{
            UE_LOGF(LogIOS, Warning, "Multiple haptics were prepared at once! Implement a stack of haptics types, or a wrapper object that is returned, with state");
			[GFeedbackGenerator release];
		}

		GHapticsType = Type;
		switch (GHapticsType)
		{
			case EMobileHapticsType::FeedbackSuccess:
			case EMobileHapticsType::FeedbackWarning:
			case EMobileHapticsType::FeedbackError:
				GFeedbackGenerator = [[UINotificationFeedbackGenerator alloc] init];
				break;

			case EMobileHapticsType::SelectionChanged:
				GFeedbackGenerator = [[UISelectionFeedbackGenerator alloc] init];
				break;

			default:
				GHapticsType = EMobileHapticsType::ImpactLight;
				// fall-through, and treat like Impact

			case EMobileHapticsType::ImpactLight:
				GFeedbackGenerator = [[UIImpactFeedbackGenerator alloc] initWithStyle:UIImpactFeedbackStyleLight];
				break;

			case EMobileHapticsType::ImpactMedium:
				GFeedbackGenerator = [[UIImpactFeedbackGenerator alloc] initWithStyle:UIImpactFeedbackStyleMedium];
				break;

			case EMobileHapticsType::ImpactHeavy:
				GFeedbackGenerator = [[UIImpactFeedbackGenerator alloc] initWithStyle:UIImpactFeedbackStyleHeavy];
				break;
		}

		// prepare the generator object so Trigger won't delay
		[GFeedbackGenerator prepare];
#endif // !PLATFORM_TVOS
	});
}

void FIOSPlatformMisc::TriggerMobileHaptics()
{
	dispatch_async(dispatch_get_main_queue(), ^
	{
#if !PLATFORM_TVOS && !PLATFORM_VISIONOS
		if (GFeedbackGenerator == nullptr)
		{
			return;
		}

		switch (GHapticsType)
		{
			case EMobileHapticsType::FeedbackSuccess:
				[(UINotificationFeedbackGenerator*)GFeedbackGenerator notificationOccurred:UINotificationFeedbackTypeSuccess];
				break;

			case EMobileHapticsType::FeedbackWarning:
				[(UINotificationFeedbackGenerator*)GFeedbackGenerator notificationOccurred:UINotificationFeedbackTypeWarning];
				break;

			case EMobileHapticsType::FeedbackError:
				[(UINotificationFeedbackGenerator*)GFeedbackGenerator notificationOccurred:UINotificationFeedbackTypeError];
				break;

			case EMobileHapticsType::SelectionChanged:
				[(UISelectionFeedbackGenerator*)GFeedbackGenerator selectionChanged];
				break;

			case EMobileHapticsType::ImpactLight:
			case EMobileHapticsType::ImpactMedium:
			case EMobileHapticsType::ImpactHeavy:
				[(UIImpactFeedbackGenerator*)GFeedbackGenerator impactOccurred];
				break;
		}
#endif // !PLATFORM_TVOS
	});
}

void FIOSPlatformMisc::ReleaseMobileHaptics()
{
	dispatch_async(dispatch_get_main_queue(), ^
	{
#if !PLATFORM_TVOS && !PLATFORM_VISIONOS
		if (GFeedbackGenerator == nullptr)
		{
			return;
		}

		[GFeedbackGenerator release];
		GFeedbackGenerator = nullptr;
#endif // !PLATFORM_TVOS
	});
}

void FIOSPlatformMisc::ShareURL(const FString& URL, const FText& Description, int32 LocationHintX, int32 LocationHintY)
{
#if WITH_APPLICATION_CORE
	NSString* SharedString = [NSString stringWithFString:Description.ToString()];
	NSURL* SharedURL = [NSURL URLWithString:[NSString stringWithFString:URL]];
    
    float ScaleFactor = (float)[IOSAppDelegate GetDelegate].IOSView.contentScaleFactor;
    __block CGRect PopoverLocation = CGRectMake(static_cast<float>(LocationHintX)/ScaleFactor, static_cast<float>(LocationHintY)/ScaleFactor, 10, 10);

	dispatch_async(dispatch_get_main_queue(),^ {
		NSArray* ObjectsToShare = @[SharedString, SharedURL];
#if !PLATFORM_TVOS
		// create the share sheet view
		UIActivityViewController* ActivityVC = [[UIActivityViewController alloc] initWithActivityItems:ObjectsToShare applicationActivities:nil];
		[ActivityVC autorelease];
	
		// skip over some things that don't make sense
		ActivityVC.excludedActivityTypes = @[UIActivityTypePrint,
											 UIActivityTypeAssignToContact,
											 UIActivityTypeSaveToCameraRoll,
											 UIActivityTypePostToFlickr,
											 UIActivityTypePostToVimeo];
		
		if ([[UIDevice currentDevice] userInterfaceIdiom] == UIUserInterfaceIdiomPhone)
		{
			[[IOSAppDelegate GetDelegate].IOSController presentViewController:ActivityVC animated:YES completion:nil];
		}
		else
		{
			// Present the view controller using the popover style.
			ActivityVC.modalPresentationStyle = UIModalPresentationPopover;
			[[IOSAppDelegate GetDelegate].IOSController presentViewController:ActivityVC
							   animated:YES
							 completion:nil];
			
			// Get the popover presentation controller and configure it.
			UIPopoverPresentationController* PresentationController = [ActivityVC popoverPresentationController];
			PresentationController.sourceView = [IOSAppDelegate GetDelegate].IOSView;
			PresentationController.sourceRect = PopoverLocation;
			
		}
#endif // !PLATFORM_TVOS
	});
#endif // WITH_APPLICATION_CORE
}


FString FIOSPlatformMisc::LoadTextFileFromPlatformPackage(const FString& RelativePath)
{
	FString FilePath = FString([[NSBundle mainBundle] bundlePath]) / RelativePath;

	// read in the command line text file (coming from UnrealFrontend) if it exists
	int32 File = open(TCHAR_TO_UTF8(*FilePath), O_RDONLY);
	if (File == -1)
	{
		LowLevelOutputDebugStringf(TEXT("No file found at %s") LINE_TERMINATOR, *FilePath);
		return FString();
	}

	ON_SCOPE_EXIT
	{
		close(File);
	};

	struct stat FileInfo;
	FileInfo.st_size = -1;

	if (fstat(File, &FileInfo))
	{
		LowLevelOutputDebugStringf(TEXT("Failed to determine file size of %s") LINE_TERMINATOR, *FilePath);
		return FString();
	}

	if (FileInfo.st_size > MAX_int32 - 1)
	{
		LowLevelOutputDebugStringf(TEXT("File too big %s") LINE_TERMINATOR, *FilePath);
		return FString();
	}

	LowLevelOutputDebugStringf(TEXT("Found %s file") LINE_TERMINATOR, *RelativePath);

	int32 FileSize = static_cast<int32>(FileInfo.st_size);
	TArray<char> FileContents;
	FileContents.AddUninitialized(FileSize + 1);
	FileContents[FileSize] = 0;

	int32 NumRead = read(File, FileContents.GetData(), FileSize);
	if (NumRead != FileSize)
	{
		LowLevelOutputDebugStringf(TEXT("Failed to read %s") LINE_TERMINATOR, *FilePath);
		return FString();
	}

	// chop off trailing spaces
	int32 Last = FileSize - 1;
	while (FileContents[0] && isspace(FileContents[Last]))
	{
		FileContents[Last] = 0;
		--Last;
	}

	return FString(UTF8_TO_TCHAR(FileContents.GetData()));
}

bool FIOSPlatformMisc::FileExistsInPlatformPackage(const FString& RelativePath)
{
	if (!FPathViews::IsRelativePath(RelativePath))
	{
		UE_LOGF(LogIOS, Warning, "FileExistsInPlatformPackage: expected a relative path, but received %ls", *RelativePath);
		return false;
	}

	FString FilePath = FString([[NSBundle mainBundle] bundlePath]) / RelativePath;

	return 0 == access(TCHAR_TO_UTF8(*FilePath), F_OK);
}

void FIOSPlatformMisc::EnableVoiceChat(bool bEnable)
{
#if WITH_APPLICATION_CORE
	return [[IOSAppDelegate GetDelegate] EnableVoiceChat:bEnable];
#endif
}

bool FIOSPlatformMisc::IsVoiceChatEnabled()
{
#if WITH_APPLICATION_CORE
	return [[IOSAppDelegate GetDelegate] IsVoiceChatEnabled];
#else
	return false;
#endif
}

bool FIOSPlatformMisc::HasRecordPermission()
{
#if WITH_APPLICATION_CORE
	return [[IOSAppDelegate GetDelegate] HasRecordPermission];
#else
	return false;
#endif
}

FIOSPlatformMisc::EIOSAuthMicrophonePermissionStatus FIOSPlatformMisc::GetMicrophonePermissionStatus()
{
#if WITH_APPLICATION_CORE
	return [[IOSAppDelegate GetDelegate] GetMicrophonePermissionStatus];
#else
	return EIOSAuthMicrophonePermissionStatus::Unknown;
#endif
}

void FIOSPlatformMisc::RegisterForRemoteNotifications()
{
#if WITH_APPLICATION_CORE
	if (FApp::IsUnattended())
	{
		return;
	}

#if !PLATFORM_TVOS
    dispatch_async(dispatch_get_main_queue(), ^{
		UNUserNotificationCenter *Center = [UNUserNotificationCenter currentNotificationCenter];
		[Center requestAuthorizationWithOptions:(UNAuthorizationOptionBadge | UNAuthorizationOptionSound | UNAuthorizationOptionAlert)
							  completionHandler:^(BOOL bGranted, NSError * _Nullable error) {
								  if (error)
								  {
									  UE_LOGF(LogIOS, Log, "Failed to register for notifications.");
								  }
								  else
								  {
									  int IsAllowed = (int32)bGranted;
                                      if (bGranted)
                                      {
										  dispatch_sync(dispatch_get_main_queue(), ^{
											  UIApplication* Application = [UIApplication sharedApplication];
											  [Application registerForRemoteNotifications];  
										  });
                                      }
									  FFunctionGraphTask::CreateAndDispatchWhenReady([IsAllowed]()
																					 {
																						 FCoreDelegates::ApplicationRegisteredForUserNotificationsDelegate.Broadcast(IsAllowed);
																					 }, TStatId(), NULL, ENamedThreads::GameThread);
									  
								  }
							  }];
    });
#endif
#endif
}

bool FIOSPlatformMisc::IsRegisteredForRemoteNotifications()
{
	return false;
}

bool FIOSPlatformMisc::IsAllowedRemoteNotifications()
{
#if !PLATFORM_TVOS && NOTIFICATIONS_ENABLED
	checkf(false, TEXT("For min iOS version >= 10 use FIOSLocalNotificationService::CheckAllowedNotifications."));
	return true;
#else
	return true;
#endif
}

void FIOSPlatformMisc::UnregisterForRemoteNotifications()
{

}

FIOSPlatformMisc::EIOSAuthNotificationStatus FIOSPlatformMisc::GetNotificationAuthorizationStatus()
{
	dispatch_semaphore_t Semaphore = dispatch_semaphore_create(0);
	static EIOSAuthNotificationStatus CurrentAuthStatus = Unknown;
	dispatch_async(dispatch_get_global_queue(QOS_CLASS_BACKGROUND, 0), ^{
		[[UNUserNotificationCenter currentNotificationCenter] getNotificationSettingsWithCompletionHandler:^(UNNotificationSettings* Settings)
		{
			switch (Settings.authorizationStatus)
			{
				case UNAuthorizationStatusNotDetermined:
					CurrentAuthStatus = NotDetermined;
					break;
				case UNAuthorizationStatusDenied:
					CurrentAuthStatus = Denied;
					break;
				case UNAuthorizationStatusAuthorized:
					CurrentAuthStatus = Authorized;
					break;
				case UNAuthorizationStatusProvisional:
					CurrentAuthStatus = Provisional;
					break;
#if !PLATFORM_TVOS
				case UNAuthorizationStatusEphemeral:
					CurrentAuthStatus = Ephemeral;
					break;
#endif
				default:
					CurrentAuthStatus = Unknown;
			}
			dispatch_semaphore_signal(Semaphore);
		}];
	});

	// wait for a result, but timeout after 1s
	dispatch_semaphore_wait(Semaphore, dispatch_time(DISPATCH_TIME_NOW, NSEC_PER_SEC));
    dispatch_release(Semaphore);
	return CurrentAuthStatus;
}

FString FIOSPlatformMisc::GetPendingActivationProtocol()
{
#if WITH_APPLICATION_CORE
	NSString* ActivationProtocol = [[IOSAppDelegate GetDelegate] GetPendingActivationProtocol];
	return FString(ActivationProtocol);
#else
	return {};
#endif
}

namespace UE::FIOSPlatformMisc::Entitlements
{

// See for more information about the Blobs
// https://opensource.apple.com/source/xnu/xnu-4570.61.1/osfmk/kern/cs_blobs.h.auto.html

typedef struct __Blob {
	uint32_t magic;
	uint32_t length;
	char data[];
} CS_GenericBlob;

typedef struct __BlobIndex {
	uint32_t type;
	uint32_t offset;
} CS_BlobIndex;

typedef struct __MultiBlob {
	uint32_t magic;
	uint32_t length;
	uint32_t count;
	CS_BlobIndex index[];
} CS_MultiBlob;

struct FEntitlementsData
{
	FEntitlementsData() = default;
	
	FEntitlementsData(void* InData, uint32 InByteSize)
		: Data(InData)
		, ByteSize(InByteSize)
	{
	}
	
	struct FDeleteByFree
	{
		void operator()(void* Ptr) const
		{
			free(Ptr);
		}
	};
	
	TUniquePtr<void, FDeleteByFree> Data;
	uint32 ByteSize = 0;
};

static FEntitlementsData TryFindEntitlementsData()
{
	// iterate through the headers to find the executable, since only the executable has the entitlements
	const struct mach_header_64* ExecutableHeader = nullptr;
	char* ImageName = nullptr;
	for (uint32_t i = 0; i < _dyld_image_count() && ExecutableHeader == nullptr; i++)
	{
		const struct mach_header_64 *Header = (struct mach_header_64 *)_dyld_get_image_header(i);
		if (Header->filetype == MH_EXECUTE)
		{
			ImageName = (char *)_dyld_get_image_name(i);
			ExecutableHeader = Header;
		}
	}

	if (ExecutableHeader == nullptr)
	{
		return {};
	}

	// verify that it's a 64bit app
	if (ExecutableHeader->magic != MH_MAGIC_64)
	{
		FPlatformMisc::LowLevelOutputDebugString(TEXT("Executable is NOT 64bit. Entitlement retrieval not supported.\n"));
		return {};
	}
	uintptr_t Cursor = (uintptr_t)ExecutableHeader + sizeof(struct mach_header_64);
	const struct linkedit_data_command *SegmentCommand = NULL;

	for (uint32_t i = 0; i < ExecutableHeader->ncmds; i++, Cursor += SegmentCommand->cmdsize)
	{
		SegmentCommand = (struct linkedit_data_command *)Cursor;

		switch (SegmentCommand->cmd)
		{
			case LC_CODE_SIGNATURE:
				FPlatformMisc::LowLevelOutputDebugString(TEXT("LC_CODE_SIGNATURE found\n"));
				break;
			default:
				continue;
		}

		const struct linkedit_data_command *DataCommand = (const struct linkedit_data_command *)SegmentCommand;

		FILE* File = fopen(ImageName, "rb");
		if (File == nullptr)
		{
			FPlatformMisc::LowLevelOutputDebugString(TEXT("Could not open binary file\n"));
			return {};
		}
		CS_MultiBlob MultiBlob;
		if (fseek(File, UInt64(DataCommand->dataoff), SEEK_SET) != 0)
		{
			UE_LOGF(LogIOS, Warning, "fseek to code signature data offset failed");
			fclose(File);
			return {};
		}
		fread(&MultiBlob, sizeof(CS_MultiBlob), 1, File);
		uint32 MultiBlobCount = ntohl(MultiBlob.count);
		CS_BlobIndex MultiBlobIndex[MultiBlobCount];
		fread(&MultiBlobIndex[0], sizeof(CS_BlobIndex) * MultiBlobCount, 1, File);

		if (__builtin_bswap32(MultiBlob.magic) != 0xfade0cc0)
		{
			fclose(File);
			return {};
		}

		for (int j = 0; j < MultiBlobCount; j++)
		{
			uint32_t CurrentOffset = DataCommand->dataoff;
			uint32_t BlobOffset = ntohl(MultiBlobIndex[j].offset);
			CS_GenericBlob Blob;
			if (fseek(File, CurrentOffset + BlobOffset, SEEK_SET) != 0)
			{
				UE_LOGF(LogIOS, Warning, "fseek to blob offset failed");
				fclose(File);
				return {};
			}
			fread(&Blob, sizeof(CS_GenericBlob), 1, File);

			if (__builtin_bswap32(Blob.magic) == 0xfade7171)
			{
				// Blob's length is magic+length+data
				uint32 DataLength = ntohl(Blob.length) - sizeof(uint32_t) * 2;
				char* Data = (char*)malloc(DataLength);
				fread(&Data[0], sizeof(char) * DataLength, 1, File);
				fclose(File);
				return FEntitlementsData(Data, DataLength);
			}
			else
			{
				continue;
			}
		}
		fclose(File);
	}
	return {};
}

static bool IsPresentInEmbeddedProvision(const char *EntitlementsToFind)
{
	NSString* MobileProvisionPath = [[[NSBundle mainBundle] bundlePath] stringByAppendingPathComponent:@"embedded.mobileprovision"];

	FILE* File = fopen([MobileProvisionPath cStringUsingEncoding:1],"rb");

	size_t EntitlementsLen = strlen(EntitlementsToFind);

	if (File == NULL)
	{
		UE_LOGF(LogIOS, Warning, "Mobile Provision not found");
		return false;
	}

	size_t ReadCount;
	char Buffer[1025]; //room for null
	while(true)
	{
		ReadCount = fread(Buffer, 1, 1024, File);
		if (ReadCount <= 0)
		{
			break;
		}
		for(size_t i = 0; i < ReadCount; i++)
		{
			if(Buffer[i] == 0)
			{
				Buffer[i] = ' ';  // replace any null terminators that might be in the binary data
			}
		}

		Buffer[ReadCount] = 0; // null terminate buffer!
		char* EntitlementsKeyPtr = strstr(Buffer, EntitlementsToFind);

		if(EntitlementsKeyPtr)
		{
			if (fseek(File, -ReadCount + EntitlementsKeyPtr - Buffer, SEEK_CUR) != 0) // seek to immediately after the entitlements key
			{
				UE_LOGF(LogIOS, Warning, "fseek to entitlements key position failed");
				fclose(File);
				return false;
			}
			ReadCount = fread(Buffer, 1, 1024, File); // read 1024 bytes immediately after the entitlements key so we definitely get the value untruncated
			// we don't expect any \0 characters between the key and the value, so we don't do the replace thing.
			Buffer[ReadCount] = 0; // null terminate buffer!

			// there could be keys following our one with their own true or false values that we could accidentally match if we just look for 'true' and return immediately.
			char* TruePtr = strstr(Buffer, "true");
			char* FalsePtr = strstr(Buffer, "false");

			if(TruePtr == NULL && FalsePtr == NULL)
			{
				FPlatformMisc::LowLevelOutputDebugString(TEXT("Unexpected Behaviour. The entitlement key is found but its value is not set.\n"));
				return false;
			}
			if(TruePtr && FalsePtr == NULL)  // only true
			{
				FPlatformMisc::LowLevelOutputDebugString(TEXT("Entitlements found in embedded mobile provision file.\n"));
				return true;
			}
			if(TruePtr == NULL && FalsePtr) // only false
			{
				FPlatformMisc::LowLevelOutputDebugString(TEXT("Entitlements found but set to false.\n"));
				return false;
			}
			return (TruePtr < FalsePtr); // return true if true comes before false
		}

		if(ReadCount < 1024)
		{
			break; // end of file
		}

		// seek back in case the entitlements was truncated.
		if (fseek(File, -EntitlementsLen, SEEK_CUR) != 0)
		{
			UE_LOGF(LogIOS, Warning, "fseek for entitlements rewind failed");
			fclose(File);
			return false;
		}
	}
	fclose(File);
	return false;
}

static uint32 StripWhitespaceInline(ANSICHAR* Data, uint32 Length)
{
	check(Data);

	ANSICHAR* Write = Data;
	const ANSICHAR* Read    = Data;
	const ANSICHAR* ReadEnd = Read + Length;
	
	while (Read != ReadEnd)
	{
		if (!TChar<ANSICHAR>::IsWhitespace(*Read))
		{
			*Write++ = *Read;
		}
		++Read;
	}
	
	check(Read >= Write);
	uint32 NewLength = Length - (Read - Write);
	
	return NewLength;
}

static bool IsPresentInCodeSignature(const char* EntitlementToCheck)
{
	check(EntitlementToCheck);
	
	FEntitlementsData RawEntitlements = TryFindEntitlementsData();
	
	bool bFound = false;

	if (RawEntitlements.Data)
	{
		ANSICHAR* Data = reinterpret_cast<ANSICHAR*>(RawEntitlements.Data.Get());
		uint32 Length = StripWhitespaceInline(Data, RawEntitlements.ByteSize);
		
		static_assert(sizeof(ANSICHAR) == sizeof(UTF8CHAR));
		FUtf8StringView Entitlements = FUtf8StringView(reinterpret_cast<UTF8CHAR*>(Data), Length);
		
		const FUtf8StringView Needle = EntitlementToCheck;
		const int32 Index = Entitlements.Find(Needle);
		if (Index != INDEX_NONE)
		{
			Entitlements.RemovePrefix(Index + Needle.Len());
			bFound = Entitlements.StartsWith("</key><true/>");
		}
	}
	else
	{
		UE_LOGF(LogIOS, Log, "No Entitlements found");
	}
	
	return bFound;
}

} // namespace UE::FIOSPlatformMisc::Entitlements

bool FIOSPlatformMisc::IsEntitlementEnabled(const char* EntitlementToCheck)
{
	check(EntitlementToCheck);
	
	bool bFound = UE::FIOSPlatformMisc::Entitlements::IsPresentInCodeSignature(EntitlementToCheck);
	
    if (!bFound)
    {
		FPlatformMisc::LowLevelOutputDebugString(TEXT("Entitlements not found in binary Mach-O header. Looking at the embedded mobile provision file.\n"));
        return UE::FIOSPlatformMisc::Entitlements::IsPresentInEmbeddedProvision(EntitlementToCheck);
    }

	return true;
}

void FIOSPlatformMisc::GetValidTargetPlatforms(TArray<FString>& TargetPlatformNames)
{
	// this is only used to cook with the proper TargetPlatform with COTF, it's not the runtime platform (which is just IOS for both)
#if PLATFORM_TVOS
	TargetPlatformNames.Add(TEXT("TVOS"));
#elif PLATFORM_VISIONOS
	TargetPlatformNames.Add(TEXT("VISIONOS"));
#else
	TargetPlatformNames.Add(FIOSPlatformProperties::PlatformName());
#endif
}

FString FIOSPlatformMisc::GetCPUVendor()
{
	return TEXT("Apple");
}

FString FIOSPlatformMisc::GetCPUBrand()
{
	return GetIOSDeviceIDString();
}

void FIOSPlatformMisc::GetOSVersions(FString& out_OSVersionLabel, FString& out_OSSubVersionLabel)
{
#if PLATFORM_TVOS
	out_OSVersionLabel = TEXT("TVOS");
#elif PLATFORM_VISIONOS
	out_OSVersionLabel = TEXT("VisionOS");
#else
	out_OSVersionLabel = TEXT("IOS");
#endif
	NSOperatingSystemVersion IOSVersion;
	IOSVersion = [[NSProcessInfo processInfo] operatingSystemVersion];
	out_OSSubVersionLabel = FString::Printf(TEXT("%ld.%ld.%ld"), IOSVersion.majorVersion, IOSVersion.minorVersion, IOSVersion.patchVersion);
}

int32 FIOSPlatformMisc::IOSVersionCompare(uint8 Major, uint8 Minor, uint8 Revision)
{
	NSOperatingSystemVersion IOSVersion;
	IOSVersion = [[NSProcessInfo processInfo] operatingSystemVersion];
	uint8 TargetValues[3] = { Major, Minor, Revision };
	NSInteger ComponentValues[3] = { IOSVersion.majorVersion, IOSVersion.minorVersion, IOSVersion.patchVersion };

	for (uint32 i = 0; i < 3; i++)
	{
		if (ComponentValues[i] < TargetValues[i])
		{
			return -1;
		}
		else if (ComponentValues[i] > TargetValues[i])
		{
			return 1;
		}
	}

	return 0;
}

FString FIOSPlatformMisc::GetProjectVersion()
{
	NSDictionary* infoDictionary = [[NSBundle mainBundle] infoDictionary];
	FString localVersionString = FString(infoDictionary[@"CFBundleShortVersionString"]);
	return localVersionString;
}

FString FIOSPlatformMisc::GetBuildNumber()
{
	NSDictionary* infoDictionary = [[NSBundle mainBundle]infoDictionary];
	FString BuildString = FString(infoDictionary[@"CFBundleVersion"]);
	return BuildString;
}

bool FIOSPlatformMisc::IsBackgroundAppRefreshAvailable()
{
	return (UIBackgroundRefreshStatusAvailable == [[UIApplication sharedApplication] backgroundRefreshStatus]);
}

void FIOSPlatformMisc::OpenAppNotificationSettings()
{
	dispatch_async(dispatch_get_main_queue(), ^{
		NSURL * SettingsUrl = [[NSURL alloc]initWithString:UIApplicationOpenNotificationSettingsURLString];
		[[UIApplication sharedApplication]openURL:SettingsUrl options:@{} completionHandler:nil];
		[SettingsUrl release];
	});
}

void FIOSPlatformMisc::OpenAppCustomSettings()
{
	dispatch_async(dispatch_get_main_queue(), ^{
		NSURL * SettingsUrl = [[NSURL alloc]initWithString:UIApplicationOpenSettingsURLString];
		[[UIApplication sharedApplication]openURL:SettingsUrl options:@{} completionHandler:nil];
		[SettingsUrl release];
	});
}

bool FIOSPlatformMisc::RequestDeviceCheckToken(TFunction<void(const TArray<uint8>&)> QuerySucceededFunc, TFunction<void(const FString&, const FString&)> QueryFailedFunc)
{
	DCDevice* DeviceCheckDevice = [DCDevice currentDevice];
	if ([DeviceCheckDevice isSupported])
	{
		[DeviceCheckDevice generateTokenWithCompletionHandler : ^ (NSData * _Nullable token, NSError * _Nullable error)
		{
			bool bSuccess = (error == NULL);
			if (bSuccess)
			{
				TArray<uint8> DeviceToken((uint8*)[token bytes], [token length]);

				QuerySucceededFunc(DeviceToken);
			}
			else
			{
				FString ErrorDescription([error localizedDescription]);

				NSDate* currentDate = [[[NSDate alloc] init] autorelease];
                NSTimeZone* timeZone = [NSTimeZone defaultTimeZone];
                NSDateFormatter* dateFormatter = [[[NSDateFormatter alloc] init] autorelease];
                [dateFormatter setTimeZone:timeZone];
                [dateFormatter setDateFormat:@"yyyy-mm-dd'T'HH:mm:ss.SSS'Z'"];
                FString localDateString([dateFormatter stringFromDate:currentDate]);
                
				QueryFailedFunc(ErrorDescription, localDateString);
			}
		}];

		return true;
	}

	return false;
}

void (*GCrashHandlerPointer)(const FGenericCrashContext& Context) = NULL;

// good enough default crash reporter
static void DefaultCrashHandler(FIOSCrashContext const& Context)
{
    Context.ReportCrash();
    if (GLog)
    {
        GLog->Panic();
    }
    if (GWarn)
    {
        GWarn->Flush();
    }
    if (GError)
    {
        GError->Flush();
        GError->HandleError();
    }
    return Context.GenerateCrashInfo();
}

// true system specific crash handler that gets called first
static FIOSCrashContext TempCrashContext(ECrashContextType::Crash, TEXT("Temp Context"));
static void PlatformCrashHandler(int32 Signal, siginfo_t* Info, void* Context)
{
	// switch to crash handler malloc to avoid malloc reentrancy
	check(FIOSApplicationInfo::CrashMalloc);
	FIOSApplicationInfo::CrashMalloc->Enable(&TempCrashContext, FPlatformTLS::GetCurrentThreadId());
	
    FIOSCrashContext CrashContext(ECrashContextType::Crash, TEXT("Caught signal"));
    CrashContext.InitFromSignal(Signal, Info, Context);
	
	// switch to the crash malloc to the new context now that we have everything
	FIOSApplicationInfo::CrashMalloc->SetContext(&CrashContext);
	
    if (GCrashHandlerPointer)
    {
        GCrashHandlerPointer(CrashContext);
    }
    else
    {
        // call default one
        DefaultCrashHandler(CrashContext);
    }
}

static void PLCrashReporterHandler(siginfo_t* Info, ucontext_t* Uap, void* Context)
{
    PlatformCrashHandler((int32)Info->si_signo, Info, Uap);
}

// handles graceful termination.
static void GracefulTerminationHandler(int32 Signal, siginfo_t* Info, void* Context)
{
    // make sure we write out as much as possible
    if (GLog)
    {
        GLog->Panic();
    }
    if (GWarn)
    {
        GWarn->Flush();
    }
    if (GError)
    {
        GError->Flush();
    }
    
    if (!IsEngineExitRequested())
    {
		RequestEngineExit(TEXT("iOS GracefulTerminationHandler"));
    }
    else
    {
        _Exit(0);
    }
}

static TMap<FString, FString> GetPredefinedConfigRuleVars()
{
	TMap<FString, FString> PredefinedConfigRuleVars;
	const FPlatformMemoryConstants& MemConstants = FPlatformMemory::GetConstants();
	uint64 PhysMemBytes = [NSProcessInfo processInfo].physicalMemory;
	int32 totalMemoryMB = (int32)(MemConstants.TotalPhysical / 1024 / 1024);
	int32 totalMemoryGB = (int32)(MemConstants.TotalPhysicalGB);	

	FString CpuVendor = FIOSPlatformMisc::GetCPUVendor();
	FString CPUBrand = FIOSPlatformMisc::GetCPUBrand();
	FString ProjectVersion = FIOSPlatformMisc::GetProjectVersion();
	FString BuildNumber = FIOSPlatformMisc::GetBuildNumber();
	FString OSVersionLabel,OSSubVersionLabel;
	FIOSPlatformMisc::GetOSVersions(OSVersionLabel,OSSubVersionLabel);
	
	PredefinedConfigRuleVars.Add(FString("SRC_OSVersion"), OSSubVersionLabel);
	PredefinedConfigRuleVars.Add(FString("SRC_OSLanguage"), *FString([[NSLocale preferredLanguages] objectAtIndex:0]));
	
	PredefinedConfigRuleVars.Add(FString("SRC_GPUFamily"), CpuVendor);
	PredefinedConfigRuleVars.Add(FString("SRC_DeviceMake"), CpuVendor);
	PredefinedConfigRuleVars.Add(FString("SRC_DeviceModel"), CPUBrand);
	PredefinedConfigRuleVars.Add(FString("SRC_TotalPhysicalGB"), FString::FromInt(totalMemoryGB));
	PredefinedConfigRuleVars.Add(FString("memory"), FString::FromInt(totalMemoryMB));

	PredefinedConfigRuleVars.Add(FString("versionCode"), BuildNumber);
	PredefinedConfigRuleVars.Add(FString("versionName"), ProjectVersion);
	
	PredefinedConfigRuleVars.Add(FString("processorCount"), FString::FromInt(FIOSPlatformMisc::NumberOfCores()));
#if !PLATFORM_TVOS && !PLATFORM_VISIONOS
	id<MTLDevice> DefaultMetalDevice = MTLCreateSystemDefaultDevice();
	if(DefaultMetalDevice)
	{
		FString GpuName([DefaultMetalDevice name]);
		PredefinedConfigRuleVars.Add(FString("SRC_GPUFamily"), GpuName);
		MTLGPUFamily Families[] = {
			MTLGPUFamilyApple1,
			MTLGPUFamilyApple2,
			MTLGPUFamilyApple3,
			MTLGPUFamilyApple4,
			MTLGPUFamilyApple5,
			MTLGPUFamilyApple6,
			MTLGPUFamilyApple7,
			MTLGPUFamilyApple8,
			MTLGPUFamilyApple9
		};
		
		for(int i = 0; i< UE_ARRAY_COUNT(Families); i++)
		{
			FString SupportsGPUFamily = FString::FromInt([DefaultMetalDevice supportsFamily:Families[i]]);
			PredefinedConfigRuleVars.Add(FString::Format(TEXT("SRC_SupportsGPUFamily{0}"), {i+1}), SupportsGPUFamily);
		}
	}
	[DefaultMetalDevice release];
#endif
	return PredefinedConfigRuleVars;
}

const TMap<FString, FString>& FIOSPlatformMisc::GetConfigRuleVars()
{
#if WITH_APPLICATION_CORE
	return FIOSConfigRules::GetConfigRulesMap();
#else
	static TMap<FString, FString> Map;
	return Map;
#endif
}

void FIOSPlatformMisc::PlatformPreInit()
{
    FGenericPlatformMisc::PlatformPreInit();
    
    GIOSAppInfo.Init();

    // turn off SIGPIPE crashes
    signal(SIGPIPE, SIG_IGN);

#if WITH_APPLICATION_CORE
	FIOSConfigRules::Init(GetPredefinedConfigRuleVars());
#endif
}

// Make sure that SetStoredValue and GetStoredValue generate the same key
static NSString* MakeStoredValueKeyName(const FString& SectionName, const FString& KeyName)
{
	return [NSString stringWithFString:(SectionName + "/" + KeyName)];
}

bool FIOSPlatformMisc::SetStoredValue(const FString& InStoreId, const FString& InSectionName, const FString& InKeyName, const FString& InValue)
{
	NSUserDefaults* UserSettings = [NSUserDefaults standardUserDefaults];

	// convert input to an NSString
	NSString* StoredValue = [NSString stringWithFString:InValue];

	// store it
	[UserSettings setObject:StoredValue forKey:MakeStoredValueKeyName(InSectionName, InKeyName)];

	return true;
}

bool FIOSPlatformMisc::GetStoredValue(const FString& InStoreId, const FString& InSectionName, const FString& InKeyName, FString& OutValue)
{
	NSUserDefaults* UserSettings = [NSUserDefaults standardUserDefaults];

	// get the stored NSString
	NSString* StoredValue = [UserSettings objectForKey:MakeStoredValueKeyName(InSectionName, InKeyName)];

	// if it was there, convert back to FString
	if (StoredValue != nil)
	{
		OutValue = StoredValue;
		return true;
	}

	return false;
}

bool FIOSPlatformMisc::DeleteStoredValue(const FString& InStoreId, const FString& InSectionName, const FString& InKeyName)
{
	NSUserDefaults* UserSettings = [NSUserDefaults standardUserDefaults];
	
	// Remove it
	[UserSettings removeObjectForKey:MakeStoredValueKeyName(InSectionName, InKeyName)];

	return true;
}

bool FIOSPlatformMisc::DeleteStoredSection(const FString& InStoreId, const FString& InSectionName)
{
	bool bRemoved = false;
	NSUserDefaults* UserSettings = [NSUserDefaults standardUserDefaults];
	NSDictionary<NSString*,id>* KeyValues = [UserSettings dictionaryRepresentation];
	NSString* SectionName = [NSString stringWithFString:InSectionName];

	for (id Key in KeyValues)
	{
		if ([Key hasPrefix:SectionName])
		{
			[UserSettings removeObjectForKey:Key];
			bRemoved = true;
		}
	}

	return bRemoved;
}

void FIOSPlatformMisc::SetGracefulTerminationHandler()
{
    struct sigaction Action;
    FMemory::Memzero(&Action, sizeof(struct sigaction));
    Action.sa_sigaction = GracefulTerminationHandler;
    sigemptyset(&Action.sa_mask);
    Action.sa_flags = SA_SIGINFO | SA_RESTART | SA_ONSTACK;
    sigaction(SIGINT, &Action, NULL);
    sigaction(SIGTERM, &Action, NULL);
    sigaction(SIGHUP, &Action, NULL);
}

void FIOSPlatformMisc::SetCrashHandler(void (* CrashHandler)(const FGenericCrashContext& Context))
{
    SCOPED_AUTORELEASE_POOL;
    
    GCrashHandlerPointer = CrashHandler;
    
#if !PLATFORM_TVOS && !PLATFORM_VISIONOS
    if (!FIOSApplicationInfo::CrashReporter && !FIOSApplicationInfo::CrashMalloc)
    {
        // configure the crash handler malloc zone to reserve a little memory for itself
        FIOSApplicationInfo::CrashMalloc = new FIOSMallocCrashHandler(4*1024*1024);
        
        PLCrashReporterConfig* Config = [[[PLCrashReporterConfig alloc] initWithSignalHandlerType: PLCrashReporterSignalHandlerTypeBSD symbolicationStrategy: PLCrashReporterSymbolicationStrategyNone crashReportFolder: FIOSApplicationInfo::TemporaryCrashReportFolder().GetNSString() crashReportName: FIOSApplicationInfo::TemporaryCrashReportName().GetNSString()] autorelease];
        FIOSApplicationInfo::CrashReporter = [[PLCrashReporter alloc] initWithConfiguration: Config];
        
        PLCrashReporterCallbacks CrashReportCallback = {
            .version = 0,
            .context = nullptr,
            .handleSignal = PLCrashReporterHandler
        };
        
        [FIOSApplicationInfo::CrashReporter setCrashCallbacks: &CrashReportCallback];
        
        NSError* Error = nil;
        if ([FIOSApplicationInfo::CrashReporter enableCrashReporterAndReturnError: &Error])
        {
            /* no-op */
        }
        else
        {
            UE_LOGF(LogIOS, Log, "Failed to enable PLCrashReporter: %ls", *FString([Error localizedDescription]));
            UE_LOGF(LogIOS, Log, "Falling back to native signal handlers");
 
            struct sigaction Action;
            FMemory::Memzero(&Action, sizeof(struct sigaction));
            Action.sa_sigaction = PlatformCrashHandler;
            sigemptyset(&Action.sa_mask);
            Action.sa_flags = SA_SIGINFO | SA_RESTART | SA_ONSTACK;
            sigaction(SIGQUIT, &Action, NULL);
            sigaction(SIGILL, &Action, NULL);
            sigaction(SIGEMT, &Action, NULL);
            sigaction(SIGFPE, &Action, NULL);
            sigaction(SIGBUS, &Action, NULL);
            sigaction(SIGSEGV, &Action, NULL);
            sigaction(SIGSYS, &Action, NULL);
            sigaction(SIGABRT, &Action, NULL);
        }
    }
#endif
}

bool FIOSPlatformMisc::HasSeparateChannelForDebugOutput()
{
#if UE_BUILD_SHIPPING && !ENABLE_PGO_PROFILE
    return false;
#else
    // We should not just check if we are being debugged because you can use the Xcode log even for
    // apps launched outside the debugger.
    return true;
#endif
}

void FIOSPlatformMisc::RequestExit(bool Force, const TCHAR* CallSite)
{
#if ENABLE_PGO_PROFILE
	if (!GIsCriticalError)
	{
		// Write the PGO profiling file on a clean shutdown.
		extern void PGO_WriteFile();
		PGO_WriteFile();
	}
#endif

	if (Force)
	{
		FApplePlatformMisc::RequestExit(Force, CallSite);
	}
	else
	{
		// ForceExit is sort of a misnomer here.  This will exit the engine loop before calling _Exit() from the app delegate
#if WITH_APPLICATION_CORE
		[[IOSAppDelegate GetDelegate] ForceExit];
#endif
	}
}

void FIOSPlatformMisc::RequestExitWithStatus(bool Force, uint8 ReturnCode, const TCHAR* CallSite)
{
	if (Force)
	{
		FApplePlatformMisc::RequestExit(Force, CallSite);
	}
	else
	{
		// Implementation will ignore the return code - this may be important, so warn.
		UE_LOGF(LogIOS, Warning, "FIOSPlatformMisc::RequestExitWithStatus(%i, %d, %ls) - return code will be ignored by the generic implementation.",
			Force, ReturnCode, CallSite ? CallSite : TEXT("<NoCallSiteInfo>"));

		// ForceExit is sort of a misnomer here.  This will exit the engine loop before calling _Exit() from the app delegate
#if WITH_APPLICATION_CORE
		[[IOSAppDelegate GetDelegate] ForceExit];
#endif
	}
}

int32 FIOSPlatformMisc::GetMaxRefreshRate()
{
#if PLATFORM_VISIONOS
	return 60;
#else
	return [UIScreen mainScreen].maximumFramesPerSecond;
#endif
}

void FIOSPlatformMisc::GPUAssert()
{
    // make this a fatal error that ends here not in the log
    // changed to 3 from NULL because clang noticed writing to NULL and warned about it
	UE_FORCE_CRASH_AT_OFFSET(13);
}

void FIOSPlatformMisc::MetalAssert()
{
    // make this a fatal error that ends here not in the log
    // changed to 3 from NULL because clang noticed writing to NULL and warned about it
	UE_FORCE_CRASH_AT_OFFSET(7);
}

struct FCPUFeatures
{
	// CRC instructions support is available on Apple A10 and beyond.
	bool bHasCrc : 1 = false;
	// AES instructions support is available on Apple A7  and beyond.
	// A8 is minspec, so assuming it is always available.
	bool bHasAes : 1 = true;

	FCPUFeatures()
	{
		int32 Value = 0;
		size_t Size = sizeof(Value);
		// https://developer.apple.com/documentation/kernel/1387446-sysctlbyname/determining_instruction_set_characteristics
		if (sysctlbyname("hw.optional.armv8_crc32", &Value, &Size, nullptr, 0) == 0)
		{
			bHasCrc = Value != 0;
		}
		// AES support could be checked by hw.optional.arm.FEAT_AES.
	}
};

static FCPUFeatures DetectCPUFeatures()
{
	static FCPUFeatures Features;
	return Features;
}

bool FIOSPlatformMisc::CPUHasHwCrcSupport()
{
	return DetectCPUFeatures().bHasCrc;
}

bool FIOSPlatformMisc::CPUHasHwAesSupport()
{
	return DetectCPUFeatures().bHasAes;
}

FString FIOSPlatformMisc::GetDiscardableCacheDir()
{
	FString CachesDirectory = FString([NSSearchPathForDirectoriesInDomains(NSCachesDirectory, NSUserDomainMask, YES) objectAtIndex:0]) + TEXT("/");
	return CachesDirectory;
}

#if !UE_BUILD_SHIPPING
bool FIOSPlatformMisc::IsConsoleOpen()
{
	return GDebugConsoleOpen;
}
#endif

static FCriticalSection EnsureLock;
static bool bReentranceGuard = false;

void ReportEnsure( const TCHAR* ErrorMessage, int NumStackFramesToIgnore )
{
    // Simple re-entrance guard.
    EnsureLock.Lock();
    
    if( bReentranceGuard )
    {
        EnsureLock.Unlock();
        return;
    }
    
    bReentranceGuard = true;
    
#if !PLATFORM_TVOS && !PLATFORM_VISIONOS
    if(FIOSApplicationInfo::CrashReporter != nil)
    {
        siginfo_t Signal;
        Signal.si_signo = SIGTRAP;
        Signal.si_code = TRAP_TRACE;
        Signal.si_addr = __builtin_return_address(0);
        
        FIOSCrashContext EnsureContext(ECrashContextType::Ensure, ErrorMessage);
        EnsureContext.InitFromSignal(SIGTRAP, &Signal, nullptr);
        EnsureContext.GenerateEnsureInfo();
    }
#endif
    
    bReentranceGuard = false;
    EnsureLock.Unlock();
}

FString FIOSCrashContext::CreateCrashFolder() const
{
	// create a crash-specific directory
	char CrashInfoFolder[PATH_MAX] = {};
	FCStringAnsi::Strncpy(CrashInfoFolder, GIOSAppInfo.CrashReportPath, PATH_MAX);
	FCStringAnsi::StrncatTruncateDest(CrashInfoFolder, PATH_MAX, "/CrashReport-UE-");
	FCStringAnsi::StrncatTruncateDest(CrashInfoFolder, PATH_MAX, GIOSAppInfo.AppNameUTF8);
	FCStringAnsi::StrncatTruncateDest(CrashInfoFolder, PATH_MAX, "-pid-");
	FCStringAnsi::StrncatTruncateDest(CrashInfoFolder, PATH_MAX, ItoANSI(getpid(), 10));
	FCStringAnsi::StrncatTruncateDest(CrashInfoFolder, PATH_MAX, "-");
	FCStringAnsi::StrncatTruncateDest(CrashInfoFolder, PATH_MAX, ItoANSI(GIOSAppInfo.RunUUID.A, 16));
	FCStringAnsi::StrncatTruncateDest(CrashInfoFolder, PATH_MAX, ItoANSI(GIOSAppInfo.RunUUID.B, 16));
	FCStringAnsi::StrncatTruncateDest(CrashInfoFolder, PATH_MAX, ItoANSI(GIOSAppInfo.RunUUID.C, 16));
	FCStringAnsi::StrncatTruncateDest(CrashInfoFolder, PATH_MAX, ItoANSI(GIOSAppInfo.RunUUID.D, 16));
	
	return FString(ANSI_TO_TCHAR(CrashInfoFolder));
}


class FIOSExec : public FSelfRegisteringExec
{
public:
	FIOSExec()
		: FSelfRegisteringExec()
	{
		
	}
	
	virtual bool Exec(UWorld* Inworld, const TCHAR* Cmd, FOutputDevice& Ar) override
	{
		if (FParse::Command(&Cmd, TEXT("IOS")))
		{
			// commands to override and append commandline options for next boot (see FIOSCommandLineHelper)
			if (FParse::Command(&Cmd, TEXT("OverrideCL")))
			{
				return FPlatformMisc::SetStoredValue(TEXT(""), TEXT("IOSCommandLine"), TEXT("ReplacementCL"), Cmd);
			}
			else if (FParse::Command(&Cmd, TEXT("AppendCL")))
			{
				return FPlatformMisc::SetStoredValue(TEXT(""), TEXT("IOSCommandLine"), TEXT("AppendCL"), Cmd);
			}
			else if (FParse::Command(&Cmd, TEXT("ClearAllCL")))
			{
				return FPlatformMisc::DeleteStoredValue(TEXT(""), TEXT("IOSCommandLine"), TEXT("ReplacementCL")) &&
						FPlatformMisc::DeleteStoredValue(TEXT(""), TEXT("IOSCommandLine"), TEXT("AppendCL"));
			}
		}
		
		return false;
	}
} GIOSExec;

float FIOSPlatformMisc::GetVirtualKeyboardInputHeight()
{
	return GIOSVirtualKeyboardInputFieldHeight.load(std::memory_order_relaxed);
}
