// Copyright Epic Games, Inc. All Rights Reserved.

#include "IOS/IOSPlatformOutputDevices.h"
#include "Misc/OutputDeviceFile.h"
#include "GenericPlatform/GenericPlatformFile.h"



class FOutputDevice*	FIOSPlatformOutputDevices::GetLog()
{
    static FIOSOutputDeviceFile Singleton(nullptr, false);
    return &Singleton;
}

//class FFeedbackContext*				FIOSPlatformOutputDevices::GetWarn()
//{
//	static FOutputDeviceIOSDebug Singleton;
//	return &Singleton;
//}

FIOSOutputDeviceFile::FIOSOutputDeviceFile(const TCHAR* InFilename /*= nullptr*/, bool bDisableBackup /*= false*/, bool bAppendIfExists /*= false*/)
	: FOutputDeviceFile(InFilename, bDisableBackup, bAppendIfExists)
{
}


void FIOSOutputDeviceFile::Serialize( const TCHAR* Data, ELogVerbosity::Type Verbosity, const class FName& Category, const double Time )
{
	// Create logfiles in the public Documents folder even if other files are going into the private Library folder
	IPlatformFile& PlatformFile = IPlatformFile::GetPlatformPhysical();
	bool bPrevCreatePublicFiles = (&PlatformFile)->DoesCreatePublicFiles();
	(&PlatformFile)->SetCreatePublicFiles(true);
	
	FOutputDeviceFile::Serialize(Data, Verbosity, Category, Time);
	
	(&PlatformFile)->SetCreatePublicFiles(bPrevCreatePublicFiles);
}
