// Copyright Epic Games, Inc. All Rights Reserved.

#include "Containers/UnrealString.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformMisc.h"
#include "Logging/LogMacros.h"
#include "Misc/Paths.h"
#include "Misc/CommandLine.h"
#include "Misc/Parse.h"

#if PLATFORM_COMPILER_OPTIMIZATION_PG_PROFILING
extern "C" void __llvm_profile_reset_counters(void);
extern "C" int  __llvm_profile_write_file(void);
extern "C" void __llvm_profile_set_filename(char*);

DEFINE_LOG_CATEGORY_STATIC(LogPGO, Log, Log)

static uint64 PGOFileCounter = 0;
FString PGO_GetOutputDirectory()
{
	FString PGOOutputDirectory;
	if (FParse::Value(FCommandLine::Get(), TEXT("pgoprofileoutput="), PGOOutputDirectory))
	{
		return PGOOutputDirectory;
	}

	FString DefaultPGODir = FPaths::ConvertRelativePathToFull(FPaths::ProjectSavedDir());
	DefaultPGODir = IFileManager::Get().ConvertToAbsolutePathForExternalAppForWrite(*DefaultPGODir);
	IFileManager::Get().MakeDirectory(*DefaultPGODir, true);
	
	UE_LOG(LogPGO, Warning, TEXT("No PGO output destination path specifed, defaulting to %s"), *DefaultPGODir);
	return DefaultPGODir;
}

static void PGO_ResetCounters()
{
	// Reset all counters, essentially restarting the profile run.

	UE_LOG(LogPGO, Log, TEXT("Resetting PGO counters."));
	__llvm_profile_reset_counters();
}

void PGO_WriteFile()
{
	static const FString OutputDirectory = PGO_GetOutputDirectory();
	FString OutputFileName = FPaths::Combine(OutputDirectory, FString::Printf(TEXT("%d.profraw"), ++PGOFileCounter));

	UE_LOG(LogPGO, Log, TEXT("Writing out PGO results file: \"%s\"."), *OutputFileName);
	FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Writing out PGO results file: \"%s\""), *OutputFileName);

	__llvm_profile_set_filename(TCHAR_TO_ANSI(*OutputFileName));

	if (__llvm_profile_write_file() != 0)
	{
		UE_LOG(LogPGO, Error, TEXT("Failed to write PGO output file."));
	}
	else
	{
		UE_LOG(LogPGO, Log, TEXT("PGO results file written successfully."));
	}

	// Reset counters after writing a file so we don't count the
	// profiling data twice if another file is written out.
	PGO_ResetCounters();
}
#endif // PLATFORM_COMPILER_OPTIMIZATION_PG_PROFILING
