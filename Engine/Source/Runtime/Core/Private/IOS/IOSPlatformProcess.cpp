// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
IOSPlatformProcess.cpp: iOS implementations of Process functions
=============================================================================*/

#include "IOS/IOSPlatformProcess.h"

#include "Apple/ScopeAutoreleasePool.h"
#include "Apple/AppleStringUtils.h"

#if PLATFORM_MAC
#include "Mac/MacSystemIncludes.h"
#elif PLATFORM_IOS
#include "IOS/IOSSystemIncludes.h"
#endif

#include "Apple/ApplePlatformRunnableThread.h"
#include "IOS/IOSAsyncTask.h"
#include "Misc/CoreDelegates.h"
#include "Apple/PreAppleSystemHeaders.h"
#include <mach-o/dyld.h>
#include "Apple/PostAppleSystemHeaders.h"

// numbers recommended by Apple
#define GAME_THREAD_PRIORITY 47
#define RENDER_THREAD_PRIORITY 45


const TCHAR* FIOSPlatformProcess::ComputerName()
{
	static TCHAR Result[256]=TEXT("");

	if( !Result[0] )
	{
		ANSICHAR AnsiResult[UE_ARRAY_COUNT(Result)];
		gethostname(AnsiResult, UE_ARRAY_COUNT(Result));
		FCString::Strcpy(Result, ANSI_TO_TCHAR(AnsiResult));
	}
	return Result;
}

const TCHAR* FIOSPlatformProcess::BaseDir()
{
	return TEXT("");
}

FRunnableThread* FIOSPlatformProcess::CreateRunnableThread()
{
	return new FRunnableThreadApple();
}

void FIOSPlatformProcess::LaunchURL( const TCHAR* URL, const TCHAR* Parms, FString* Error )
{
	UE_LOGF(LogIOS, Log,  "LaunchURL %ls %ls", URL, Parms?Parms:TEXT("") );

	if (FCoreDelegates::ShouldLaunchUrl.IsBound() && !FCoreDelegates::ShouldLaunchUrl.Execute(URL))
	{
		if (Error)
		{
			*Error = TEXT("LaunchURL cancelled by delegate");
		}
		return;
	}

	FString SchemeName;
	bool bHasSchemeName = FParse::SchemeNameFromURI(URL, SchemeName);
		
	NSString* CFUrl = (NSString*)FPlatformString::TCHARToCFString( URL );
	if(!bHasSchemeName)
	{
		CFUrl = [@"http://" stringByAppendingString:CFUrl];
	}
	[[UIApplication sharedApplication] openURL: [NSURL URLWithString: CFUrl] options:@{} completionHandler:
		^(BOOL Success)
		{
			if (!Success)
			{
				UE_LOGF(LogIOS, Warning, "LaunchURL of %ls was NOT successful", URL);
			}
		}
	];
	
	if (Error != nullptr)
	{
		*Error = TEXT("");
	}
}

/**
* Opens the specified URL and waits for it to open, up to the Timeout (s).  If there's an error with the opening
* that's found within the timeout period, or a timeout occurs, Error is set to "Unable to open url"
*
* @param URL The url to open
* @param Parms Not currently used
* @param Timeout The length of time to wait (in seconds) before it times out
* @param Error Either blank or with an error message if fails to open or times out
**/
void FIOSPlatformProcess::LaunchSyncURL( const TCHAR* URL, const TCHAR* Parms, float Timeout, FString* Error )
{
	UE_LOGF(LogIOS, Log, "LauncSynchURL %ls %ls", URL, Parms?Parms:TEXT("") );

	if (FCoreDelegates::ShouldLaunchUrl.IsBound() && !FCoreDelegates::ShouldLaunchUrl.Execute(URL))
	{
		if (Error)
		{
			*Error = TEXT("LauncSynchURL cancelled by delegate");
		}
		return;
	}

	if (Timeout <= 0)
	{
		if (Error)
		{
			*Error = TEXT("LauncSynchURL invalid Timeout specified");
		}
		return;
	}

	dispatch_semaphore_t Semaphore = dispatch_semaphore_create(0);
	__block BOOL Result;
	FString SchemeName;
	bool bHasSchemeName = FParse::SchemeNameFromURI(URL, SchemeName);
		
	NSString* CFUrl = (NSString*)FPlatformString::TCHARToCFString( URL );
	if(!bHasSchemeName)
	{
		CFUrl = [@"http://" stringByAppendingString:CFUrl];
	}
	[[UIApplication sharedApplication] openURL: [NSURL URLWithString: CFUrl] options:@{} completionHandler:
		^(BOOL Success)
		{
			Result = Success;
			dispatch_semaphore_signal(Semaphore);
		}
	];
	
	// wait for a result, up to "Timeout" seconds
	if (dispatch_semaphore_wait(Semaphore, dispatch_time(DISPATCH_TIME_NOW, (uint64_t)(Timeout*(double)NSEC_PER_SEC))))
	{
		// We timed out, error unknown
		Result = TRUE;
	}
	dispatch_release(Semaphore);
	
	if (Error != nullptr)
	{
		*Error = Result ? TEXT("") : TEXT("Unable to open url");
	}
}

bool FIOSPlatformProcess::CanLaunchURL(const TCHAR* URL)
{
	NSString* CFUrl = (NSString*)FPlatformString::TCHARToCFString(URL);
    bool Result = [[UIApplication sharedApplication] canOpenURL:[NSURL URLWithString: CFUrl]];
	return Result;
}

FString FIOSPlatformProcess::GetGameBundleId()
{
	return FString([[NSBundle mainBundle] bundleIdentifier]);
}

// Set the game thread priority to very high, slightly above the render thread
void FIOSPlatformProcess::SetupThread(const int Priority)
{
	struct sched_param Sched;
	FMemory::Memzero(&Sched, sizeof(struct sched_param));

	// Read the current priority and policy
	int32 CurrentPolicy = SCHED_RR;
	pthread_getschedparam(pthread_self(), &CurrentPolicy, &Sched);

	// Set the new priority and policy (apple recommended FIFO for the two main non-working threads)
	int32 Policy = SCHED_FIFO;
	Sched.sched_priority = Priority;
	pthread_setschedparam(pthread_self(), Policy, &Sched);
}

void FIOSPlatformProcess::SetupGameThread()
{
	SetupThread(GAME_THREAD_PRIORITY);
}

void FIOSPlatformProcess::SetupRenderThread()
{
	SetupThread(RENDER_THREAD_PRIORITY);
}

void FIOSPlatformProcess::SetThreadAffinityMask(uint64 AffinityMask)
{
	if (FPlatformMisc::NumberOfCores() > 1)
	{
		thread_affinity_policy AP;
		AP.affinity_tag = AffinityMask;
		thread_policy_set(pthread_mach_thread_np(pthread_self()), THREAD_AFFINITY_POLICY, (integer_t*)&AP, THREAD_AFFINITY_POLICY_COUNT);
	}
	else
	{
		FGenericPlatformProcess::SetThreadAffinityMask(AffinityMask);
	}
}

const TCHAR* FIOSPlatformProcess::ExecutablePath()
{
	static TCHAR Result[512]=TEXT("");
	if( !Result[0] )
	{
		SCOPED_AUTORELEASE_POOL;
		NSString *NSExeName = [[NSBundle mainBundle] executablePath];
		FPlatformString::CFStringToTCHAR( ( CFStringRef )NSExeName, Result );
	}
	return Result;
}

const TCHAR* FIOSPlatformProcess::ExecutableName(bool bRemoveExtension)
{
	static TCHAR Result[512] = TEXT("");
	if (!Result[0])
	{
		NSString *NSExeName = [[[NSBundle mainBundle] executablePath] lastPathComponent];
		FPlatformString::CFStringToTCHAR((CFStringRef)NSExeName, Result);
	}
	return Result;
}

FString FIOSPlatformProcess::GenerateApplicationPath( const FString& AppName, EBuildConfiguration BuildConfiguration)
{
    SCOPED_AUTORELEASE_POOL;
    
    FString PlatformName = "IOS";
    FString ExecutableName = AppName;
    if (BuildConfiguration != EBuildConfiguration::Development && BuildConfiguration != EBuildConfiguration::DebugGame)
    {
        ExecutableName += FString::Printf(TEXT("-%s-%s"), *PlatformName, LexToString(BuildConfiguration));
    }
    
    NSURL* CurrentBundleURL = [[NSBundle mainBundle] bundleURL];
    NSString* CurrentBundleName = [[CurrentBundleURL lastPathComponent] stringByDeletingPathExtension];
    if(FString(CurrentBundleName) == ExecutableName)
    {
        CFStringRef FilePath = CFURLCopyFileSystemPath((CFURLRef)CurrentBundleURL, kCFURLPOSIXPathStyle);
        FString ExecutablePath = FString::Printf(TEXT("%s/%s"), *FString((NSString*)FilePath), *ExecutableName);
        CFRelease(FilePath);
        return ExecutablePath;
    }
    else
    {
        return FString();
    }
}

const uint64 FIOSPlatformAffinity::GetMainGameMask()
{
	return MAKEAFFINITYMASK1(0);
}

const uint64 FIOSPlatformAffinity::GetRenderingThreadMask()
{
	static int Mask = 0;
	if (Mask == 0)
	{
		switch (FPlatformMisc::NumberOfCores())
		{
		case 2:
		case 3:
			Mask = MAKEAFFINITYMASK1(1);
			break;
		default:
			Mask = FGenericPlatformAffinity::GetRenderingThreadMask();
			break;
		}
	}

	return Mask;
}

const uint64 FIOSPlatformAffinity::GetRTHeartBeatMask()
{
	static int Mask = 0;
	if (Mask == 0)
	{
		switch (FPlatformMisc::NumberOfCores())
		{
		case 2:
		case 3:
			Mask = MAKEAFFINITYMASK1(0);
			break;
		default:
			Mask = FGenericPlatformAffinity::GetRTHeartBeatMask();
			break;
		}
	}

	return Mask;
}

const uint64 FIOSPlatformAffinity::GetPoolThreadMask()
{
	static int Mask = 0;
	if (Mask == 0)
	{
		switch (FPlatformMisc::NumberOfCores())
		{
		case 2:
			Mask = MAKEAFFINITYMASK1(1);
			break;
		case 3:
			Mask = MAKEAFFINITYMASK1(2);
			break;
		default:
			Mask = FGenericPlatformAffinity::GetPoolThreadMask();
			break;
		}
	}

	return Mask;
}

const uint64 FIOSPlatformAffinity::GetTaskGraphThreadMask()
{
	static int Mask = 0;
	if (Mask == 0)
	{
		switch (FPlatformMisc::NumberOfCores())
		{
		case 2:
			Mask = MAKEAFFINITYMASK1(1);
			break;
		case 3:
			Mask = MAKEAFFINITYMASK1(2);
			break;
		default:
			Mask = FGenericPlatformAffinity::GetTaskGraphThreadMask();
			break;
		}
	}

	return Mask;
}

const uint64 FIOSPlatformAffinity::GetNoAffinityMask()
{
	static int Mask = 0;
	if (Mask == 0)
	{
		switch (FPlatformMisc::NumberOfCores())
		{
		case 2:
		case 3:
			Mask = (1 << FPlatformMisc::NumberOfCores()) - 1;
			break;
		default:
			Mask = FGenericPlatformAffinity::GetNoAffinityMask();
			break;
		}
	}

	return Mask;
}

