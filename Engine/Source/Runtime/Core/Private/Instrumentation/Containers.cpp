// Copyright Epic Games, Inc. All Rights Reserved.

#include "Instrumentation/Containers.h"

#if PLATFORM_WINDOWS && USING_INSTRUMENTATION

#include "Windows.h"

void* FInstrumentationSafeWinAllocator::Alloc(size_t Size)
{
	return VirtualAlloc(NULL, Size, MEM_RESERVE | MEM_COMMIT, PAGE_READWRITE);
}

void FInstrumentationSafeWinAllocator::Free(void* Data)
{
	VirtualFree(Data, 0, MEM_RELEASE);
}

INSTRUMENTATION_FUNCTION_ATTRIBUTES static size_t GetPageSize()
{
	static size_t PageSize = 0;
	if (PageSize == 0)
	{
		SYSTEM_INFO si;
		GetSystemInfo(&si);
		PageSize = si.dwPageSize;
	}
	return PageSize;
}

void* FInstrumentationSafeWinAllocator::AllocWithGuards(size_t Size)
{
	size_t PageSize = GetPageSize();
	size_t RequestedSize = Align(Size, PageSize);

	// Requires 1 page on each side of the allocation that won't be committed and will fault if accessed
	size_t TotalSize = PageSize * 2 + Align(Size, PageSize);
	void* BaseAddress = VirtualAlloc(NULL, TotalSize, MEM_RESERVE, PAGE_NOACCESS);
	return VirtualAlloc((void*)((UPTRINT)BaseAddress + PageSize), RequestedSize, MEM_COMMIT, PAGE_READWRITE);
}

void FInstrumentationSafeWinAllocator::FreeWithGuards(void* Data)
{
	VirtualFree((void*)((UPTRINT)Data - GetPageSize()), 0, MEM_RELEASE);
}

void* FInstrumentationSafeWinAllocator::Realloc(void* Data, size_t Size, size_t PreviousSize)
{
	if (Data == nullptr)
	{
		return Alloc(Size);
	}

	if (Size == 0)
	{
		Free(Data);
		return nullptr;
	}

	uint8* NewMem = (uint8*)Alloc(Size);
	if (NewMem)
	{
		memcpy(NewMem, Data, PreviousSize);
		Free(Data);
	}

	return NewMem;
}

#endif // PLATFORM_WINDOWS && USING_INSTRUMENTATION