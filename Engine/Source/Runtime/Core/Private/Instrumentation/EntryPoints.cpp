// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/CoreMiscDefines.h"

// For monolithic build, we include the per module inline only once in the core module.
#if IS_MONOLITHIC && USING_INSTRUMENTATION
#include "Instrumentation/PerModuleInline.inl"
#endif