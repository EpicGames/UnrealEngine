// Copyright Epic Games, Inc. All Rights Reserved.

#include "Internationalization/CamelCaseBreakIterator.h"

#include "Misc/AssertionMacros.h"
#include "Templates/UnrealTemplate.h"

FCamelCaseBreakIterator::FCamelCaseBreakIterator()
	: CurrentPosition(0)
{
	BreakPoints.Add(0);
}

void FCamelCaseBreakIterator::SetString(FString&& InString)
{
	InternalString = MoveTemp(InString);
	String = InternalString;
	UpdateBreakPointsArray();
	ResetToBeginning();
}

void FCamelCaseBreakIterator::SetStringRef(FStringView InString)
{
	InternalString.Reset();
	String = InString;
	UpdateBreakPointsArray();
	ResetToBeginning();
}

int32 FCamelCaseBreakIterator::GetCurrentPosition() const
{
	return CurrentPosition;
}

int32 FCamelCaseBreakIterator::ResetToBeginning()
{
	return CurrentPosition = 0;
}

int32 FCamelCaseBreakIterator::ResetToEnd()
{
	return CurrentPosition = String.Len();
}

int32 FCamelCaseBreakIterator::MoveToPrevious()
{
	return MoveToCandidateBefore(CurrentPosition);
}

int32 FCamelCaseBreakIterator::MoveToNext()
{
	return MoveToCandidateAfter(CurrentPosition);
}

int32 FCamelCaseBreakIterator::MoveToCandidateBefore(const int32 InIndex)
{
	CurrentPosition = InIndex;

	for(int32 BreakPointIndex = BreakPoints.Num() - 1; BreakPointIndex >= 0; --BreakPointIndex)
	{
		const int32 BreakPoint = BreakPoints[BreakPointIndex];
		if(BreakPoint < InIndex)
		{
			CurrentPosition = BreakPoint;
			break;
		}
	}

	return (CurrentPosition >= InIndex) ? INDEX_NONE : CurrentPosition;
}

int32 FCamelCaseBreakIterator::MoveToCandidateAfter(const int32 InIndex)
{
	CurrentPosition = InIndex;

	for(int32 BreakPointIndex = 0; BreakPointIndex < BreakPoints.Num(); ++BreakPointIndex)
	{
		const int32 BreakPoint = BreakPoints[BreakPointIndex];
		if(BreakPoint > InIndex)
		{
			CurrentPosition = BreakPoint;
			break;
		}
	}

	return (CurrentPosition <= InIndex) ? INDEX_NONE : CurrentPosition;
}

void FCamelCaseBreakIterator::UpdateBreakPointsArray()
{
	FTokensArray Tokens;
	TokenizeString(Tokens);
	PopulateBreakPointsArray(Tokens);
}

void FCamelCaseBreakIterator::PopulateBreakPointsArray(const FTokensArray& InTokens)
{
	BreakPoints.Empty(InTokens.Num());

	// Process the tokens so that input like "ICUBreakIterator1234_Ext" would produce the following break points:
	// ICU|Break|Iterator|1234|_|Ext|

	BreakPoints.Add(0); // start of the string

	auto AddBreakPoint = [this](int32 StrIndex)
	{
		if (StrIndex > BreakPoints.Last())
		{
			BreakPoints.Add(StrIndex);
		}
	};

	ETokenType TokenRunType = ETokenType::Other;
	for(int32 TokenIndex = 0; TokenIndex < InTokens.Num(); ++TokenIndex)
	{
		const FToken& Token = InTokens[TokenIndex];

		// End of string?
		if(Token.TokenType == ETokenType::Null)
		{
			AddBreakPoint(Token.StrIndex);
			break;
		}

		// Digits behave specially when around character tokens so that strings like "D3D11Func" and "Vector2dToString" break as would be expected ("D3D11|Func", and "Vector2d|To|String")
		// We handle this by remapping the token run type under certain circumstances to avoid incorrectly breaking the run
		if(TokenRunType == ETokenType::Digit || TokenRunType == ETokenType::Uppercase || TokenRunType == ETokenType::Lowercase)
		{
			if((Token.TokenType == ETokenType::Digit) != (TokenRunType == ETokenType::Digit))
			{
				TokenRunType = Token.TokenType;
			}
		}

		// Have we found the end of some kind of run of tokens?
		if(TokenRunType != Token.TokenType)
		{
			// If we've moved from a run of upper-case tokens, to a lower-case token, then we need to try and make the previous upper-case token part of the next run
			const int32 BreakTokenIndex = TokenIndex - ((TokenRunType == ETokenType::Uppercase && Token.TokenType == ETokenType::Lowercase) ? 1 : 0);
			if(BreakTokenIndex > 0)
			{
				AddBreakPoint(InTokens[BreakTokenIndex].StrIndex);
			}
		}

		// Always add "other" tokens as break points
		if(Token.TokenType == ETokenType::Other)
		{
			AddBreakPoint(Token.StrIndex);
		}

		TokenRunType = Token.TokenType;
	}

	// There should always be at least one entry for the end of the string
	check(BreakPoints.Num());
}
