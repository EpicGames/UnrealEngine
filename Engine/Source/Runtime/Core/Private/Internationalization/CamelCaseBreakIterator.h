// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/StringFwd.h"
#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Internationalization/IBreakIterator.h"

/**
 * Base class for camel case break iterators
 * A derived type must provide a way to tokenize the string for processing
 */
class FCamelCaseBreakIterator : public IBreakIterator
{
public:
	FCamelCaseBreakIterator();

	virtual void SetString(FString&& InString) override;
	virtual void SetStringRef(FStringView InString) override;

	virtual int32 GetCurrentPosition() const override;

	virtual int32 ResetToBeginning() override;
	virtual int32 ResetToEnd() override;

	virtual int32 MoveToPrevious() override;
	virtual int32 MoveToNext() override;
	virtual int32 MoveToCandidateBefore(const int32 InIndex) override;
	virtual int32 MoveToCandidateAfter(const int32 InIndex) override;

protected:
	enum class ETokenType : uint8
	{
		Uppercase,
		Lowercase,
		Digit,
		Null,
		Other,
	};

	struct FToken
	{
		FToken(const ETokenType InTokenType, const int32 InStrIndex)
			: TokenType(InTokenType)
			, StrIndex(InStrIndex)
		{
		}

		ETokenType TokenType;
		int32 StrIndex;
	};

	using FTokensArray = TArray<FToken, TInlineAllocator<1024>>;
	virtual void TokenizeString(FTokensArray& OutTokens) = 0;

protected:
	void UpdateBreakPointsArray();
	void PopulateBreakPointsArray(const FTokensArray& InTokens);

	FString InternalString;
	FStringView String;
	int32 CurrentPosition;
	TArray<int32, TInlineAllocator<32>> BreakPoints;
};
