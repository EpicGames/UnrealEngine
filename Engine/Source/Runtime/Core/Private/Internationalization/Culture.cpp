// Copyright Epic Games, Inc. All Rights Reserved.

#include "Internationalization/Culture.h"
#include "Containers/ArrayView.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/App.h"

#if UE_ENABLE_ICU
#include "Internationalization/ICUCulture.h"
#else
#include "Internationalization/LegacyCulture.h"
#endif

void ApplyCultureDisplayNameSubstitutes(TArrayView<const FString> InPrioritizedCultureNames, FString& InOutDisplayName)
{
	struct FDisplayNameSubstitute
	{
		FString Culture;
		FString OldString;
		FString NewString;
	};

	static TArray<FDisplayNameSubstitute> CultureDisplayNameSubstitutes;

	// Conditionally load the required config data
	{
		static bool bHasInitializedCultureDisplayNameSubstitutes = false;
		if (!bHasInitializedCultureDisplayNameSubstitutes && GConfig && GConfig->IsReadyForUse())
		{
			bHasInitializedCultureDisplayNameSubstitutes = true;

			TArray<FString> CultureDisplayNameSubstitutesStrArray;
			{
				const bool ShouldLoadEditor = GIsEditor;
				const bool ShouldLoadGame = FApp::IsGame();

				GConfig->GetArray(TEXT("Internationalization"), TEXT("CultureDisplayNameSubstitutes"), CultureDisplayNameSubstitutesStrArray, GEngineIni);

				if (ShouldLoadEditor)
				{
					TArray<FString> EditorArray;
					GConfig->GetArray(TEXT("Internationalization"), TEXT("CultureDisplayNameSubstitutes"), EditorArray, GEditorIni);
					CultureDisplayNameSubstitutesStrArray.Append(MoveTemp(EditorArray));
				}

				if (ShouldLoadGame)
				{
					TArray<FString> GameArray;
					GConfig->GetArray(TEXT("Internationalization"), TEXT("CultureDisplayNameSubstitutes"), GameArray, GGameIni);
					CultureDisplayNameSubstitutesStrArray.Append(MoveTemp(GameArray));
				}
			}

			// Each substitute should be a semi-colon separated set of data: [Culture;]Old;New
			CultureDisplayNameSubstitutes.Reserve(CultureDisplayNameSubstitutesStrArray.Num());
			for (const FString& CultureDisplayNameSubstituteStr : CultureDisplayNameSubstitutesStrArray)
			{
				TArray<FString> DisplayFragments;
				CultureDisplayNameSubstituteStr.ParseIntoArray(DisplayFragments, TEXT(";"));

				if (DisplayFragments.Num() == 2)
				{
					CultureDisplayNameSubstitutes.Add(FDisplayNameSubstitute{ FString(), MoveTemp(DisplayFragments[0]), MoveTemp(DisplayFragments[1]) });
				}
				else if (DisplayFragments.Num() == 3)
				{
					CultureDisplayNameSubstitutes.Add(FDisplayNameSubstitute{ MoveTemp(DisplayFragments[0]), MoveTemp(DisplayFragments[1]), MoveTemp(DisplayFragments[2]) });
				}
			}

			// Sort by culture name length, so that more specific cultures get first refusal at a replacement
			CultureDisplayNameSubstitutes.StableSort([](const FDisplayNameSubstitute& InOne, const FDisplayNameSubstitute& InTwo)
			{
				return InOne.Culture.Len() > InTwo.Culture.Len();
			});
		}
	}

	for (const FDisplayNameSubstitute& CultureDisplayNameSubstitute : CultureDisplayNameSubstitutes)
	{
		const bool bValidForCulture = CultureDisplayNameSubstitute.Culture.IsEmpty() || InPrioritizedCultureNames.Contains(CultureDisplayNameSubstitute.Culture);
		if (bValidForCulture)
		{
			InOutDisplayName.ReplaceInline(*CultureDisplayNameSubstitute.OldString, *CultureDisplayNameSubstitute.NewString, ESearchCase::CaseSensitive);
		}
	}
}

FCultureRef FCulture::Create(TUniquePtr<FCultureImplementation>&& InImplementation)
{
	check(InImplementation.IsValid());
	return MakeShareable(new FCulture(MoveTemp(InImplementation)));
}

FCulture::FCulture(TUniquePtr<FCultureImplementation>&& InImplementation)
	: Implementation(MoveTemp(InImplementation))
	, CachedName(Implementation->GetName())
	, CachedVerseIdentifier(FCulture::CultureNameToVerseIdentifier(CachedName))
	, CachedUnrealLegacyThreeLetterISOLanguageName(Implementation->GetUnrealLegacyThreeLetterISOLanguageName())
	, CachedThreeLetterISOLanguageName(Implementation->GetThreeLetterISOLanguageName())
	, CachedTwoLetterISOLanguageName(Implementation->GetTwoLetterISOLanguageName())
	, CachedRegion(Implementation->GetRegion())
	, CachedScript(Implementation->GetScript())
	, CachedVariant(Implementation->GetVariant())
	, CachedIsRightToLeft(Implementation->IsRightToLeft())
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FCulture::FCulture);

	RefreshCultureDisplayNames(TArray<FString>()); // The display name for the current language will be updated by a post-construct call to RefreshCultureDisplayNames with the correct language data
}

FCulture::~FCulture()
{
	// Explicit destructor due to forward declaration of FCultureImplementation
}

const FString& FCulture::GetDisplayName() const
{
	return CachedDisplayName;
}

const FString& FCulture::GetEnglishName() const
{
	return CachedEnglishName;
}

int FCulture::GetKeyboardLayoutId() const
{
	return Implementation->GetKeyboardLayoutId();
}

int FCulture::GetLCID() const
{
	return Implementation->GetLCID();
}

TArray<FString> FCulture::GetPrioritizedParentCultureNames() const
{
	return GetPrioritizedParentCultureNames(GetTwoLetterISOLanguageName(), GetScript(), GetRegion());
}

TArray<FString> FCulture::GetPrioritizedParentCultureNames(const FString& LanguageCode, const FString& ScriptCode, const FString& RegionCode)
{
	TArray<FString> LocaleTagCombinations;

	if (!ScriptCode.IsEmpty() && !RegionCode.IsEmpty())
	{
		LocaleTagCombinations.Add(CreateCultureName(LanguageCode, ScriptCode, RegionCode));
	}

	if (!RegionCode.IsEmpty())
	{
		LocaleTagCombinations.Add(CreateCultureName(LanguageCode, FString(), RegionCode));
	}

	if (!ScriptCode.IsEmpty())
	{
		LocaleTagCombinations.Add(CreateCultureName(LanguageCode, ScriptCode, FString()));
	}

	LocaleTagCombinations.Add(LanguageCode);

	return LocaleTagCombinations;
}

FString FCulture::CreateCultureName(const FString& LanguageCode, const FString& ScriptCode, const FString& RegionCode)
{
	if (!ScriptCode.IsEmpty() && !RegionCode.IsEmpty())
	{
		FString CultureName;
		CultureName.Reserve(LanguageCode.Len() + ScriptCode.Len() + RegionCode.Len() + 2);
		CultureName += LanguageCode;
		CultureName += TEXT('-');
		CultureName += ScriptCode;
		CultureName += TEXT('-');
		CultureName += RegionCode;
		return CultureName;
	}

	if (!RegionCode.IsEmpty())
	{
		FString CultureName;
		CultureName.Reserve(LanguageCode.Len() + RegionCode.Len() + 1);
		CultureName += LanguageCode;
		CultureName += TEXT('-');
		CultureName += RegionCode;
		return CultureName;
	}

	if (!ScriptCode.IsEmpty())
	{
		FString CultureName;
		CultureName.Reserve(LanguageCode.Len() + ScriptCode.Len() + 1);
		CultureName += LanguageCode;
		CultureName += TEXT('-');
		CultureName += ScriptCode;
		return CultureName;
	}

	return LanguageCode;
}

FString FCulture::GetCanonicalName(const FString& Name)
{
	return FCultureImplementation::GetCanonicalName(Name, FInternationalization::Get());
}

FString FCulture::CultureNameToVerseIdentifier(const FString& Name)
{
	// Verse uses "en_US" style while UE uses "en-US"
	FString Result = Name;
	Result.ReplaceCharInline(TEXT('-'), TEXT('_'), ESearchCase::CaseSensitive);
	return Result;
}

FString FCulture::CultureNameFromVerseIdentifier(const FString& VerseIdentifer)
{
	// UE uses "en-US" style while Verse uses "en_US"
	FString Result = VerseIdentifer;
	Result.ReplaceCharInline(TEXT('_'), TEXT('-'), ESearchCase::CaseSensitive);
	return Result;
}

ECultureIdentifierType FCulture::GetPreferredOnDiskIdentifierType()
{
	checkf(GConfig && GConfig->IsReadyForUse(), TEXT("FCulture::GetPreferredOnDiskIdentifierType called before the config system was initialized!"));

	static const ECultureIdentifierType IdentifierType = 
		[]()
		{
			if (FString IdentifierTypeStr;
				GConfig->GetString(TEXT("Internationalization"), TEXT("PreferredOnDiskCultureIdentifierType"), IdentifierTypeStr, GEngineIni))
			{
				if (IdentifierTypeStr == TEXTVIEW("Verse"))
				{
					return ECultureIdentifierType::Verse;
				}
			}
			return ECultureIdentifierType::UE;
		}();
	return IdentifierType;
}

const FString& FCulture::GetPreferredOnDiskIdentifier() const
{
	return GetPreferredOnDiskIdentifierType() == ECultureIdentifierType::Verse
		? GetVerseIdentifier()
		: GetName();
}

const FString& FCulture::GetName() const
{
	return CachedName;
}

const FString& FCulture::GetVerseIdentifier() const
{
	return CachedVerseIdentifier;
}

const FString& FCulture::GetNativeName() const
{
	return CachedNativeName;
}

const FString& FCulture::GetUnrealLegacyThreeLetterISOLanguageName() const
{
	return CachedUnrealLegacyThreeLetterISOLanguageName;
}

const FString& FCulture::GetThreeLetterISOLanguageName() const
{
	return CachedThreeLetterISOLanguageName;
}

const FString& FCulture::GetTwoLetterISOLanguageName() const
{
	return CachedTwoLetterISOLanguageName;
}

const FString& FCulture::GetNativeLanguage() const
{
	return CachedNativeLanguage;
}

const FString& FCulture::GetRegion() const
{
	return CachedRegion;
}

const FString& FCulture::GetNativeRegion() const
{
	return CachedNativeRegion;
}

const FString& FCulture::GetScript() const
{
	return CachedScript;
}

const FString& FCulture::GetVariant() const
{
	return CachedVariant;
}

bool FCulture::IsRightToLeft() const
{
	return CachedIsRightToLeft;
}

const FDecimalNumberFormattingRules& FCulture::GetDecimalNumberFormattingRules() const
{
	return Implementation->GetDecimalNumberFormattingRules();
}

const FDecimalNumberFormattingRules& FCulture::GetPercentFormattingRules() const
{
	return Implementation->GetPercentFormattingRules();
}

const FDecimalNumberFormattingRules& FCulture::GetCurrencyFormattingRules(const FString& InCurrencyCode) const
{
	return Implementation->GetCurrencyFormattingRules(InCurrencyCode);
}

/**
 * Get the correct plural form to use for the given number
 * ICU only supports int32 and double, so we cast larger int values to double to try and keep as much precision as possible
 */

#define DEF_GETPLURALFORM_CAST(T1, T2) ETextPluralForm FCulture::GetPluralForm(T1 Val, const ETextPluralType PluralType) const { return GetPluralForm((T2)Val, PluralType); }
DEF_GETPLURALFORM_CAST(float, double)
DEF_GETPLURALFORM_CAST(int8, int32)
DEF_GETPLURALFORM_CAST(int16, int32)
DEF_GETPLURALFORM_CAST(int64, double)
DEF_GETPLURALFORM_CAST(uint8, int32)
DEF_GETPLURALFORM_CAST(uint16, int32)
DEF_GETPLURALFORM_CAST(uint32, double)
DEF_GETPLURALFORM_CAST(uint64, double)
#undef DEF_GETPLURALFORM_CAST

ETextPluralForm FCulture::GetPluralForm(int32 Val, const ETextPluralType PluralType) const
{
	if (Val < 0)
	{
		// Must be positive
		Val *= -1;
	}
	return Implementation->GetPluralForm(Val, PluralType);
}

ETextPluralForm FCulture::GetPluralForm(double Val, const ETextPluralType PluralType) const
{
	if (Val < 0.0)
	{
		// Must be positive
		Val *= -1.0;
	}
	return Implementation->GetPluralForm(Val, PluralType);
}

const TArray<ETextPluralForm>& FCulture::GetValidPluralForms(const ETextPluralType PluralType) const
{
	return Implementation->GetValidPluralForms(PluralType);
}

void FCulture::RefreshCultureDisplayNames(const TArray<FString>& InPrioritizedDisplayCultureNames, const bool bFullRefresh)
{
	CachedDisplayName = Implementation->GetDisplayName();
	ApplyCultureDisplayNameSubstitutes(InPrioritizedDisplayCultureNames, CachedDisplayName);

	if (bFullRefresh)
	{
		{
			static const FString EnglishCultureName = "en";
			CachedEnglishName = Implementation->GetEnglishName();
			ApplyCultureDisplayNameSubstitutes(MakeArrayView(&EnglishCultureName, 1), CachedEnglishName);
		}

		{
			const TArray<FString> PrioritizedNativeCultureNames = GetPrioritizedParentCultureNames();

			CachedNativeName = Implementation->GetNativeName();
			ApplyCultureDisplayNameSubstitutes(PrioritizedNativeCultureNames, CachedNativeName);

			CachedNativeLanguage = Implementation->GetNativeLanguage();
			ApplyCultureDisplayNameSubstitutes(PrioritizedNativeCultureNames, CachedNativeLanguage);

			CachedNativeRegion = Implementation->GetNativeRegion();
			ApplyCultureDisplayNameSubstitutes(PrioritizedNativeCultureNames, CachedNativeRegion);
		}
	}
}
