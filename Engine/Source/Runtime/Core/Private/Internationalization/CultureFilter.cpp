// Copyright Epic Games, Inc. All Rights Reserved.

#include "Internationalization/CultureFilter.h"
#include "Internationalization/LocalizedTextSourceTypes.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/App.h"

FCultureFilter::FCultureFilter(const TSet<FString>* AvailableCultures)
{
	const EBuildConfiguration BuildConfig = FApp::GetBuildConfiguration();

	const ELocalizationLoadFlags TargetFlags 
		= ELocalizationLoadFlags::Engine
		| (GIsEditor ? ELocalizationLoadFlags::Editor : ELocalizationLoadFlags::None)
		| (FApp::IsGame() ? ELocalizationLoadFlags::Game : ELocalizationLoadFlags::None);

	Init(BuildConfig, TargetFlags, AvailableCultures);
}

FCultureFilter::FCultureFilter(const EBuildConfiguration BuildConfig, const ELocalizationLoadFlags TargetFlags, const TSet<FString>* AvailableCultures)
{
	Init(BuildConfig, TargetFlags, AvailableCultures);
}

void FCultureFilter::Init(const EBuildConfiguration BuildConfig, const ELocalizationLoadFlags TargetFlags, const TSet<FString>* AvailableCultures)
{
	// Get the build config string so we can compare it against the config entries
	FString BuildConfigString;
	{
		EBuildConfiguration LocalBuildConfig = BuildConfig;
		if (LocalBuildConfig == EBuildConfiguration::DebugGame)
		{
			// Treat DebugGame and Debug as the same for loc purposes
			LocalBuildConfig = EBuildConfiguration::Debug;
		}

		if (LocalBuildConfig != EBuildConfiguration::Unknown)
		{
			BuildConfigString = LexToString(LocalBuildConfig);
		}
	}

	auto LoadInternationalizationConfigArray = [TargetFlags](const TCHAR* InKey) -> TArray<FString>
	{
		check(GConfig && GConfig->IsReadyForUse());

		TArray<FString> FinalArray;

		if (EnumHasAnyFlags(TargetFlags, ELocalizationLoadFlags::Engine))
		{
			TArray<FString> EngineArray;
			GConfig->GetArray(TEXT("Internationalization"), InKey, EngineArray, GEngineIni);
			FinalArray.Append(MoveTemp(EngineArray));
		}

		if (EnumHasAnyFlags(TargetFlags, ELocalizationLoadFlags::Editor))
		{
			TArray<FString> EditorArray;
			GConfig->GetArray(TEXT("Internationalization"), InKey, EditorArray, GEditorIni);
			FinalArray.Append(MoveTemp(EditorArray));
		}

		if (EnumHasAnyFlags(TargetFlags, ELocalizationLoadFlags::Game))
		{
			TArray<FString> GameArray;
			GConfig->GetArray(TEXT("Internationalization"), InKey, GameArray, GGameIni);
			FinalArray.Append(MoveTemp(GameArray));
		}

		return FinalArray;
	};

	// An array of potentially semicolon separated mapping entries: Culture[;BuildConfig[,BuildConfig,BuildConfig]]
	// No build config(s) implies all build configs
	auto ProcessCulturesArray = [&BuildConfigString, AvailableCultures](const TArray<FString>& InCulturesArray, TSet<FString>& OutCulturesSet)
	{
		OutCulturesSet.Reserve(InCulturesArray.Num());
		for (const FString& CultureStr : InCulturesArray)
		{
			FString CultureName;
			FString CultureBuildConfigsStr;
			if (CultureStr.Split(TEXT(";"), &CultureName, &CultureBuildConfigsStr, ESearchCase::CaseSensitive))
			{
				// Check to see if any of the build configs matches our current build config
				TArray<FString> CultureBuildConfigs;
				if (CultureBuildConfigsStr.ParseIntoArray(CultureBuildConfigs, TEXT(",")))
				{
					bool bIsValidBuildConfig = false;
					for (const FString& CultureBuildConfig : CultureBuildConfigs)
					{
						if (BuildConfigString == CultureBuildConfig)
						{
							bIsValidBuildConfig = true;
							break;
						}
					}

					if (!bIsValidBuildConfig)
					{
						continue;
					}
				}
			}
			else
			{
				CultureName = CultureStr;
			}

			if (!AvailableCultures || AvailableCultures->Contains(CultureName))
			{
				OutCulturesSet.Add(MoveTemp(CultureName));
			}
			else
			{
				UE_LOGF(LogLocalization, Warning, "Culture '%ls' is unknown and has been ignored when parsing the enabled/disabled culture list.", *CultureName);
			}
		}
	};

	const TArray<FString> EnabledCulturesArray = LoadInternationalizationConfigArray(TEXT("EnabledCultures"));
	ProcessCulturesArray(EnabledCulturesArray, EnabledCultures);

	const TArray<FString> DisabledCulturesArray = LoadInternationalizationConfigArray(TEXT("DisabledCultures"));
	ProcessCulturesArray(DisabledCulturesArray, DisabledCultures);
}

bool FCultureFilter::IsCultureAllowed(const FString& Culture) const
{
	return (EnabledCultures.Num() == 0 || EnabledCultures.Contains(Culture)) && !DisabledCultures.Contains(Culture);
}
