// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"

struct FDecimalNumberFormattingRules;
enum class ETextPluralForm : uint8;
enum class ETextPluralType : uint8;

/**
 * Internal implementation of a culture.
 * @note See FCulture for the API description.
 */
class ICultureImplementation
{
public:
	virtual ~ICultureImplementation() = default;

	virtual FString GetDisplayName() const = 0;
	virtual FString GetEnglishName() const = 0;
	virtual int GetKeyboardLayoutId() const = 0;
	virtual int GetLCID() const = 0;
	virtual FString GetName() const = 0;
	virtual FString GetNativeName() const = 0;
	virtual FString GetUnrealLegacyThreeLetterISOLanguageName() const = 0;
	virtual FString GetThreeLetterISOLanguageName() const = 0;
	virtual FString GetTwoLetterISOLanguageName() const = 0;
	virtual FString GetNativeLanguage() const = 0;
	virtual FString GetNativeRegion() const = 0;
	virtual FString GetRegion() const = 0;
	virtual FString GetScript() const = 0;
	virtual FString GetVariant() const = 0;
	virtual bool IsRightToLeft() const = 0;
	virtual const FDecimalNumberFormattingRules& GetDecimalNumberFormattingRules() = 0;
	virtual const FDecimalNumberFormattingRules& GetPercentFormattingRules() = 0;
	virtual const FDecimalNumberFormattingRules& GetCurrencyFormattingRules(const FString& InCurrencyCode) = 0;
	virtual ETextPluralForm GetPluralForm(int32 Val, const ETextPluralType PluralType) const = 0;
	virtual ETextPluralForm GetPluralForm(double Val, const ETextPluralType PluralType) const = 0;
	virtual const TArray<ETextPluralForm>& GetValidPluralForms(const ETextPluralType PluralType) const = 0;
};
