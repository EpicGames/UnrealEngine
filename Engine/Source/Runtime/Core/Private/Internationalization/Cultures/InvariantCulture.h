// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if !UE_ENABLE_ICU

#include "Internationalization/Culture.h"
#include "Internationalization/LegacyCulture.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/FastDecimalFormat.h"

#define LOCTEXT_NAMESPACE "Internationalization"

class FInvariantCulture
{
public:
	static FCultureRef Create()
	{
		FDecimalNumberFormattingRules DecimalNumberFormattingRules;
		DecimalNumberFormattingRules.NaNString = TEXT("NaN");
		DecimalNumberFormattingRules.NegativePrefixString = TEXT("-");
		DecimalNumberFormattingRules.PlusString = TEXT("+");
		DecimalNumberFormattingRules.MinusString = TEXT("-");
		DecimalNumberFormattingRules.GroupingSeparatorCharacter = TEXT(',');
		DecimalNumberFormattingRules.DecimalSeparatorCharacter = TEXT('.');
		DecimalNumberFormattingRules.PrimaryGroupingSize = 3;
		DecimalNumberFormattingRules.SecondaryGroupingSize = 3;

		FDecimalNumberFormattingRules PercentFormattingRules;
		PercentFormattingRules.NaNString = TEXT("NaN");
		PercentFormattingRules.NegativePrefixString = TEXT("-");
		PercentFormattingRules.NegativeSuffixString = TEXT("%");
		PercentFormattingRules.PositiveSuffixString = TEXT("%");
		PercentFormattingRules.PlusString = TEXT("+");
		PercentFormattingRules.MinusString = TEXT("-");
		PercentFormattingRules.GroupingSeparatorCharacter = TEXT(',');
		PercentFormattingRules.DecimalSeparatorCharacter = TEXT('.');
		PercentFormattingRules.PrimaryGroupingSize = 3;
		PercentFormattingRules.SecondaryGroupingSize = 3;

		FDecimalNumberFormattingRules BaseCurrencyFormattingRules;
		BaseCurrencyFormattingRules.NaNString = TEXT("NaN");
		BaseCurrencyFormattingRules.NegativePrefixString = TEXT("-$");
		BaseCurrencyFormattingRules.PositivePrefixString = TEXT("$");
		BaseCurrencyFormattingRules.PlusString = TEXT("+");
		BaseCurrencyFormattingRules.MinusString = TEXT("-");
		BaseCurrencyFormattingRules.GroupingSeparatorCharacter = TEXT(',');
		BaseCurrencyFormattingRules.DecimalSeparatorCharacter = TEXT('.');
		BaseCurrencyFormattingRules.PrimaryGroupingSize = 3;
		BaseCurrencyFormattingRules.SecondaryGroupingSize = 3;

		FCultureRef Culture = FCulture::Create(MakeUnique<FLegacyCultureImplementation>(
			LOCTEXT("InvariantCultureDisplayName", "Invariant Language (Invariant Country)"),	//const FText DisplayName
			FString("Invariant Language (Invariant Country)"),									//const FString EnglishName
			1033,																				//const int KeyboardLayoutId
			1033,																				//const int LCID
			FString{},																			//const FString Name
			FString("Invariant Language (Invariant Country)"),									//const FString NativeName
			FString("INT"),																		//const FString UnrealLegacyThreeLetterISOLanguageName
			FString("ivl"),																		//const FString ThreeLetterISOLanguageName
			FString("iv"),																		//const FString TwoLetterISOLanguageName
			DecimalNumberFormattingRules,														//const FDecimalNumberFormattingRules InDecimalNumberFormattingRules
			PercentFormattingRules,																//const FDecimalNumberFormattingRules InPercentFormattingRules
			BaseCurrencyFormattingRules,														//const FDecimalNumberFormattingRules InBaseCurrencyFormattingRules
			false																				//const bool IsRightToLeft
		));

		return Culture;
	}
};

#undef LOCTEXT_NAMESPACE

#endif
