// Copyright Epic Games, Inc. All Rights Reserved.

#include "Internationalization/Cultures/KeysCulture.h"

#if ENABLE_LOC_TESTING

FKeysCulture::FKeysCulture(const FCultureRef& InInvariantCulture)
	: InvariantCulture(InInvariantCulture)
{
}

const FString& FKeysCulture::StaticGetName()
{
	static const FString KeysCultureName = "keys";
	return KeysCultureName;
}

#endif
