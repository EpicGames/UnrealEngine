// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/LocTesting.h"
#include "Internationalization/ICustomCulture.h"

#if ENABLE_LOC_TESTING

/** 
* A debug culture that when set will display the localization key of an FText instead of its associated text.
* This is used to help diagnose issues with wrong or missing translations in Editor/Game UI text.
* Use the command line or in game console to set this culture and it will allow you to find the localization key for an FText where you can look in the corresponding .po files to find the problematic text entry.
* The logic for performing the conversion is in FTextLocalizationManager.
* 
* @see FTextLocalizationManager
*/
class FKeysCulture : public ICustomCulture
{
public:
	explicit FKeysCulture(const FCultureRef& InInvariantCulture);

	static const FString& StaticGetName();

	//~ ICustomCulture interface
	virtual FCultureRef GetBaseCulture() const override { return InvariantCulture; }
	virtual FString GetDisplayName() const override { return FKeysCulture::StaticGetName(); }
	virtual FString GetEnglishName() const override { return FKeysCulture::StaticGetName(); }
	virtual FString GetName() const override { return FKeysCulture::StaticGetName(); }
	virtual FString GetNativeName() const override { return FKeysCulture::StaticGetName(); }
	virtual FString GetUnrealLegacyThreeLetterISOLanguageName() const override { return TEXT("INT"); }
	virtual FString GetThreeLetterISOLanguageName() const override { return FKeysCulture::StaticGetName(); }
	virtual FString GetTwoLetterISOLanguageName() const override { return FKeysCulture::StaticGetName(); }
	virtual FString GetNativeLanguage() const override { return FKeysCulture::StaticGetName(); }
	virtual FString GetNativeRegion() const override { return FString(); }
	virtual FString GetRegion() const override { return FString(); }
	virtual FString GetScript() const override { return FString(); }
	virtual FString GetVariant() const override { return FString(); }
	virtual bool IsRightToLeft() const override { return false; }

private:
	FCultureRef InvariantCulture;
};

#endif
