// Copyright Epic Games, Inc. All Rights Reserved.

#include "Internationalization/Cultures/LeetCulture.h"

#if ENABLE_LOC_TESTING

FLeetCulture::FLeetCulture(const FCultureRef& InInvariantCulture)
	: InvariantCulture(InInvariantCulture)
{
}

const FString& FLeetCulture::StaticGetName()
{
	static const FString LeetCultureName = "LEET";
	return LeetCultureName;
}

#endif
