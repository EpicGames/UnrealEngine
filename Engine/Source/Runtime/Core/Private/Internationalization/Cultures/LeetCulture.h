// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/LocTesting.h"
#include "Internationalization/ICustomCulture.h"

#if ENABLE_LOC_TESTING

class FLeetCulture : public ICustomCulture
{
public:
	explicit FLeetCulture(const FCultureRef& InInvariantCulture);

	static const FString& StaticGetName();

	//~ ICustomCulture interface
	virtual FCultureRef GetBaseCulture() const override { return InvariantCulture; }
	virtual FString GetDisplayName() const override { return FLeetCulture::StaticGetName(); }
	virtual FString GetEnglishName() const override { return FLeetCulture::StaticGetName(); }
	virtual FString GetName() const override { return FLeetCulture::StaticGetName(); }
	virtual FString GetNativeName() const override { return FLeetCulture::StaticGetName(); }
	virtual FString GetUnrealLegacyThreeLetterISOLanguageName() const override { return TEXT("INT"); }
	virtual FString GetThreeLetterISOLanguageName() const override { return FLeetCulture::StaticGetName(); }
	virtual FString GetTwoLetterISOLanguageName() const override { return FLeetCulture::StaticGetName(); }
	virtual FString GetNativeLanguage() const override { return FLeetCulture::StaticGetName(); }
	virtual FString GetNativeRegion() const override { return FString(); }
	virtual FString GetRegion() const override { return FString(); }
	virtual FString GetScript() const override { return FString(); }
	virtual FString GetVariant() const override { return FString(); }
	virtual bool IsRightToLeft() const override { return false; }

private:
	FCultureRef InvariantCulture;
};

#endif
