// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Internationalization/Culture.h"
#include "Internationalization/ICustomCulture.h"

#if UE_ENABLE_ICU
#include "Internationalization/ICUCulture.h"
#else
#include "Internationalization/LegacyCulture.h"
#endif

class FCustomCultureImplementation : public FCultureImplementation
{
public:
	FCustomCultureImplementation(const TSharedRef<ICustomCulture>& InCustomCulture)
#if UE_ENABLE_ICU
		: FCultureImplementation(InCustomCulture->GetBaseCulture()->GetName())
#else
		: FCultureImplementation(
			FText::AsCultureInvariant(InCustomCulture->GetBaseCulture()->GetDisplayName()),
			InCustomCulture->GetBaseCulture()->GetEnglishName(),
			InCustomCulture->GetBaseCulture()->GetKeyboardLayoutId(),
			InCustomCulture->GetBaseCulture()->GetLCID(),
			InCustomCulture->GetBaseCulture()->GetName(),
			InCustomCulture->GetBaseCulture()->GetNativeName(),
			InCustomCulture->GetBaseCulture()->GetUnrealLegacyThreeLetterISOLanguageName(),
			InCustomCulture->GetBaseCulture()->GetThreeLetterISOLanguageName(),
			InCustomCulture->GetBaseCulture()->GetTwoLetterISOLanguageName(),
			InCustomCulture->GetBaseCulture()->GetDecimalNumberFormattingRules(),
			InCustomCulture->GetBaseCulture()->GetPercentFormattingRules(),
			InCustomCulture->GetBaseCulture()->GetCurrencyFormattingRules(FString()),
			InCustomCulture->GetBaseCulture()->IsRightToLeft()
		)
#endif
		, CustomCulture(InCustomCulture)
	{
	}

	virtual ~FCustomCultureImplementation() = default;

	//~ ICultureImplementation interface
	virtual FString GetDisplayName() const override { return CustomCulture->GetDisplayName(); }
	virtual FString GetEnglishName() const override { return CustomCulture->GetEnglishName(); }
	virtual FString GetName() const override { return CustomCulture->GetName(); }
	virtual FString GetNativeName() const override { return CustomCulture->GetNativeName(); }
	virtual FString GetUnrealLegacyThreeLetterISOLanguageName() const override { return CustomCulture->GetUnrealLegacyThreeLetterISOLanguageName(); }
	virtual FString GetThreeLetterISOLanguageName() const override { return CustomCulture->GetThreeLetterISOLanguageName(); }
	virtual FString GetTwoLetterISOLanguageName() const override { return CustomCulture->GetTwoLetterISOLanguageName(); }
	virtual FString GetNativeLanguage() const override { return CustomCulture->GetNativeLanguage(); }
	virtual FString GetNativeRegion() const override { return CustomCulture->GetNativeRegion(); }
	virtual FString GetRegion() const override { return CustomCulture->GetRegion(); }
	virtual FString GetScript() const override { return CustomCulture->GetScript(); }
	virtual FString GetVariant() const override { return CustomCulture->GetVariant(); }
	virtual bool IsRightToLeft() const override { return CustomCulture->IsRightToLeft(); }

private:
	TSharedRef<ICustomCulture> CustomCulture;
};
