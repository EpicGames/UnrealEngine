// Copyright Epic Games, Inc. All Rights Reserved.

#include "Internationalization/FastDecimalFormat.h"

#include "Containers/StringFwd.h"
#include "HAL/IConsoleManager.h"
#include "HAL/UnrealMemory.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/AssertionMacros.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/StringBuilder.h"
#include "Templates/UnrealTemplate.h"

namespace FastDecimalFormat
{

namespace Internal
{

static const int32 MaxIntegralPrintLength = 20;
static const int32 MaxFractionalPrintPrecision = 18;
static const int32 MinRequiredIntegralBufferSize = (MaxIntegralPrintLength * 2) + 1; // *2 for an absolute worst case group separator scenario, +1 for null terminator

static int32 bFastDecimalFormatLargeFloatSupport = 1;
static FAutoConsoleVariableRef CVarFastDecimalFormatLargeFloatSupport(
	TEXT("Core.bFastDecimalFormatLargeFloatSupport"),
	bFastDecimalFormatLargeFloatSupport,
	TEXT("True implies we perform additional processing for floating point types over 9223372036854775807 to prevent clipping to this value."));

static int32 bFastDecimalFormatCanIndicateIfNearlyInteger = 1;
static FAutoConsoleVariableRef CVarIndicateIfNearlyInteger(
	TEXT("Core.bFastDecimalFormatCanIndicateIfNearlyInteger"),
	bFastDecimalFormatCanIndicateIfNearlyInteger,
	TEXT("True implies we will indicate nearly integer values with '...' - such as 1e-16 which would normally be displayed as 0."));

static const uint64 Pow10Table[] = {
	1,						// 10^0
	10,						// 10^1
	100,					// 10^2
	1000,					// 10^3
	10000,					// 10^4
	100000,					// 10^5
	1000000,				// 10^6
	10000000,				// 10^7
	100000000,				// 10^8
	1000000000,				// 10^9
	10000000000,			// 10^10
	100000000000,			// 10^11
	1000000000000,			// 10^12
	10000000000000,			// 10^13
	100000000000000,		// 10^14
	1000000000000000,		// 10^15
	10000000000000000,		// 10^16
	100000000000000000,		// 10^17
	1000000000000000000,	// 10^18
};

static_assert(UE_ARRAY_COUNT(Pow10Table) - 1 >= MaxFractionalPrintPrecision, "Pow10Table must at big enough to index any value up-to MaxFractionalPrintPrecision");

enum class EDecimalNumberSigningStringsFlags : uint8
{
	None = 0,
	AlwaysSign = 1 << 0,
	UseASCIISigns = 1 << 1,
};
ENUM_CLASS_FLAGS(EDecimalNumberSigningStringsFlags);

class FDecimalNumberSigningStrings
{
public:
	FDecimalNumberSigningStrings(const FDecimalNumberFormattingRules& InFormattingRules, const EDecimalNumberSigningStringsFlags InFlags)
	{
		// Resolve out the default cases
		NegativePrefixStringPtr = &InFormattingRules.NegativePrefixString;
		NegativeSuffixStringPtr = &InFormattingRules.NegativeSuffixString;
		PositivePrefixStringPtr = &InFormattingRules.PositivePrefixString;
		PositiveSuffixStringPtr = &InFormattingRules.PositiveSuffixString;

		// If we should always sign this number we can use the negative signing strings to synthesize a positive version
		if (EnumHasAnyFlags(InFlags, EDecimalNumberSigningStringsFlags::AlwaysSign))
		{
			auto SynthesizePositiveString = [&InFormattingRules](const FString& InNegativeString, FString& OutPositiveString) -> bool
			{
				if (InNegativeString.Contains(InFormattingRules.MinusString, ESearchCase::CaseSensitive))
				{
					OutPositiveString = InNegativeString.Replace(*InFormattingRules.MinusString, *InFormattingRules.PlusString, ESearchCase::CaseSensitive);
					return true;
				}
				return false;
			};

			if (SynthesizePositiveString(InFormattingRules.NegativePrefixString, GeneratedPositivePrefixString))
			{
				PositivePrefixStringPtr = &GeneratedPositivePrefixString;
			}
			if (SynthesizePositiveString(InFormattingRules.NegativeSuffixString, GeneratedPositiveSuffixString))
			{
				PositiveSuffixStringPtr = &GeneratedPositiveSuffixString;
			}
		}

		// If we should use an ASCII '+' and '-' then make that substitution after synthesizing the positive string
		if (EnumHasAnyFlags(InFlags, EDecimalNumberSigningStringsFlags::UseASCIISigns))
		{
			GeneratedNegativePrefixString = GetNegativePrefixString().Replace(*InFormattingRules.MinusString, TEXT("-"), ESearchCase::CaseSensitive);
			GeneratedNegativeSuffixString = GetNegativeSuffixString().Replace(*InFormattingRules.MinusString, TEXT("-"), ESearchCase::CaseSensitive);
			GeneratedPositivePrefixString = GetPositivePrefixString().Replace(*InFormattingRules.PlusString,  TEXT("+"), ESearchCase::CaseSensitive);
			GeneratedPositiveSuffixString = GetPositiveSuffixString().Replace(*InFormattingRules.PlusString,  TEXT("+"), ESearchCase::CaseSensitive);

			if (!InFormattingRules.NegativePrefixString.IsEmpty())
			{
				NegativePrefixStringPtr = &GeneratedNegativePrefixString;
			}
			if (!InFormattingRules.NegativeSuffixString.IsEmpty())
			{
				NegativeSuffixStringPtr = &GeneratedNegativeSuffixString;
			}
			if (!InFormattingRules.PositivePrefixString.IsEmpty())
			{
				PositivePrefixStringPtr = &GeneratedPositivePrefixString;
			}
			if (!InFormattingRules.PositiveSuffixString.IsEmpty())
			{
				PositiveSuffixStringPtr = &GeneratedPositiveSuffixString;
			}
		}
	}

	const FString& GetNegativePrefixString() const
	{
		return *NegativePrefixStringPtr;
	}

	const FString& GetNegativeSuffixString() const
	{
		return *NegativeSuffixStringPtr;
	}

	const FString& GetPositivePrefixString() const
	{
		return *PositivePrefixStringPtr;
	}

	const FString& GetPositiveSuffixString() const
	{
		return *PositiveSuffixStringPtr;
	}

private:
	const FString* NegativePrefixStringPtr;
	const FString* NegativeSuffixStringPtr;
	const FString* PositivePrefixStringPtr;
	const FString* PositiveSuffixStringPtr;

	FString GeneratedNegativePrefixString;
	FString GeneratedNegativeSuffixString;
	FString GeneratedPositivePrefixString;
	FString GeneratedPositiveSuffixString;
};

void SanitizeNumberFormattingOptions(FNumberFormattingOptions& InOutFormattingOptions)
{
	// Ensure that the minimum limits are >= 0
	InOutFormattingOptions.MinimumIntegralDigits = FMath::Max(0, InOutFormattingOptions.MinimumIntegralDigits);
	InOutFormattingOptions.MinimumFractionalDigits = FMath::Max(0, InOutFormattingOptions.MinimumFractionalDigits);

	// Ensure that the maximum limits are >= the minimum limits
	InOutFormattingOptions.MaximumIntegralDigits = FMath::Max(InOutFormattingOptions.MinimumIntegralDigits, InOutFormattingOptions.MaximumIntegralDigits);
	InOutFormattingOptions.MaximumFractionalDigits = FMath::Max(InOutFormattingOptions.MinimumFractionalDigits, InOutFormattingOptions.MaximumFractionalDigits);
}

int32 IntegralToString_UInt64ToString(
	const uint64 InVal, 
	const bool InUseGrouping, const uint8 InPrimaryGroupingSize, const uint8 InSecondaryGroupingSize, const uint8 InMinimumGroupingDigits, const TCHAR InGroupingSeparatorCharacter, const TCHAR* InDigitCharacters,
	const int32 InMinDigitsToPrint, const int32 InMaxDigitsToPrint, 
	TCHAR* InBufferToFill, const int32 InBufferToFillSize
	)
{
	check(InBufferToFillSize >= MinRequiredIntegralBufferSize);

	TCHAR TmpBuffer[MinRequiredIntegralBufferSize];
	int32 StringLen = 0;

	int32 DigitsPrinted = 0;
	uint8 NumUntilNextGroup = InPrimaryGroupingSize;

	if (InVal > 0)
	{
		// Perform the initial number -> string conversion
		uint64 TmpNum = InVal;
		while (DigitsPrinted < InMaxDigitsToPrint && TmpNum != 0)
		{
			if (InUseGrouping && NumUntilNextGroup-- == 0)
			{
				TmpBuffer[StringLen++] = InGroupingSeparatorCharacter;
				NumUntilNextGroup = InSecondaryGroupingSize - 1; // -1 to account for the digit we're about to print
			}

			TmpBuffer[StringLen++] = InDigitCharacters[TmpNum % 10];
			TmpNum /= 10;

			++DigitsPrinted;
		}
	}

	// Pad the string to the min digits requested
	{
		const int32 PaddingToApply = FMath::Min(InMinDigitsToPrint - DigitsPrinted, MaxIntegralPrintLength - DigitsPrinted);
		for (int32 PaddingIndex = 0; PaddingIndex < PaddingToApply; ++PaddingIndex)
		{
			if (InUseGrouping && NumUntilNextGroup-- == 0)
			{
				TmpBuffer[StringLen++] = InGroupingSeparatorCharacter;
				NumUntilNextGroup = InSecondaryGroupingSize;
			}

			TmpBuffer[StringLen++] = InDigitCharacters[0];

			++DigitsPrinted;
		}
	}

	// Did we actually write enough digits to warrant a group separator? If not then we need to remove it again
	if (InUseGrouping && DigitsPrinted > static_cast<int32>(InPrimaryGroupingSize) && DigitsPrinted < static_cast<int32>(InPrimaryGroupingSize + InMinimumGroupingDigits))
	{
		FMemory::Memmove(TmpBuffer + InPrimaryGroupingSize, TmpBuffer + InPrimaryGroupingSize + 1, StringLen - InPrimaryGroupingSize - 1);
		--StringLen;
	}

	// TmpBuffer is backwards, flip it into the final output buffer
	for (int32 FinalBufferIndex = 0; FinalBufferIndex < StringLen; ++FinalBufferIndex)
	{
		InBufferToFill[FinalBufferIndex] = TmpBuffer[StringLen - FinalBufferIndex - 1];
	}
	InBufferToFill[StringLen] = TEXT('\0');

	return StringLen;
}

FORCEINLINE int32 IntegralToString_Common(const uint64 InVal, const FDecimalNumberFormattingRules& InFormattingRules, const FNumberFormattingOptions& InFormattingOptions, TCHAR* InBufferToFill, const int32 InBufferToFillSize)
{
	// Perform the initial format to a decimal string
	return IntegralToString_UInt64ToString(
		InVal, 
		InFormattingOptions.UseGrouping && InFormattingRules.PrimaryGroupingSize > 0,
		InFormattingRules.PrimaryGroupingSize, 
		InFormattingRules.SecondaryGroupingSize, 
		InFormattingRules.MinimumGroupingDigits, 
		InFormattingRules.GroupingSeparatorCharacter, 
		InFormattingRules.DigitCharacters, 
		InFormattingOptions.MinimumIntegralDigits, 
		InFormattingOptions.MaximumIntegralDigits, 
		InBufferToFill, 
		InBufferToFillSize
		);
}

void FractionalToString_SplitAndRoundNumber(const bool bIsNegative, const double InValue, const int32 InNumDecimalPlaces, ERoundingMode InRoundingMode, double& OutIntegralPart, double& OutFractionalPart)
{
	const int32 DecimalPlacesToRoundTo = FMath::Min(InNumDecimalPlaces, MaxFractionalPrintPrecision);

	const bool bIsRoundingEntireNumber = DecimalPlacesToRoundTo == 0;

	// We split the value before performing the rounding to avoid losing precision during the rounding calculations
	// If we're rounding to zero decimal places, then we just apply rounding to the number as a whole
	double IntegralPart = InValue;
	double FractionalPart = (bIsRoundingEntireNumber) ? 0.0 : FMath::Modf(InValue, &IntegralPart);

	// Multiply the value to round by 10^DecimalPlacesToRoundTo - this will allow us to perform rounding calculations 
	// that correctly trim any remaining fractional parts that are outside of our rounding range
	double& ValueToRound = ((bIsRoundingEntireNumber) ? IntegralPart : FractionalPart);
	ValueToRound = FMath::TruncateToHalfIfClose(ValueToRound * (double)Pow10Table[DecimalPlacesToRoundTo]);

	// The rounding modes here mimic those of ICU. See http://userguide.icu-project.org/formatparse/numbers/rounding-modes
	switch (InRoundingMode)
	{
	case ERoundingMode::HalfToEven:
		// Rounds to the nearest place, equidistant ties go to the value which is closest to an even value: 1.5 becomes 2, 0.5 becomes 0
		ValueToRound = FMath::RoundHalfToEven(ValueToRound);
		break;

	case ERoundingMode::HalfFromZero:
		// Rounds to nearest place, equidistant ties go to the value which is further from zero: -0.5 becomes -1.0, 0.5 becomes 1.0
		ValueToRound = FMath::RoundHalfFromZero(ValueToRound);
		break;
	
	case ERoundingMode::HalfToZero:
		// Rounds to nearest place, equidistant ties go to the value which is closer to zero: -0.5 becomes 0, 0.5 becomes 0
		ValueToRound = FMath::RoundHalfToZero(ValueToRound);
		break;

	case ERoundingMode::FromZero:
		// Rounds to the value which is further from zero, "larger" in absolute value: 0.1 becomes 1, -0.1 becomes -1
		ValueToRound = FMath::RoundFromZero(ValueToRound);
		break;
	
	case ERoundingMode::ToZero:
		// Rounds to the value which is closer to zero, "smaller" in absolute value: 0.1 becomes 0, -0.1 becomes 0
		ValueToRound = FMath::RoundToZero(ValueToRound);
		break;
	
	case ERoundingMode::ToNegativeInfinity:
		// Rounds to the value which is more negative: 0.1 becomes 0, -0.1 becomes -1
		ValueToRound = FMath::RoundToNegativeInfinity(ValueToRound);
		break;
	
	case ERoundingMode::ToPositiveInfinity:
		// Rounds to the value which is more positive: 0.1 becomes 1, -0.1 becomes 0
		ValueToRound = FMath::RoundToPositiveInfinity(ValueToRound);
		break;

	default:
		break;
	}

	// Copy to the correct output param depending on whether we were rounding to the number as a whole
	if (bIsRoundingEntireNumber)
	{
		OutIntegralPart = ValueToRound;
		OutFractionalPart = 0.0;
	}
	else
	{
		// Rounding may have caused the fractional value to overflow, and any overflow will need to be applied to the integral part and stripped from the fractional part
		const double ValueToOverflowTest = (bIsNegative) ? -ValueToRound : ValueToRound;
		if (ValueToOverflowTest >= (double)Pow10Table[DecimalPlacesToRoundTo])
		{
			if (bIsNegative)
			{
				IntegralPart -= 1;
				ValueToRound += (double)Pow10Table[DecimalPlacesToRoundTo];
			}
			else
			{
				IntegralPart += 1;
				ValueToRound -= (double)Pow10Table[DecimalPlacesToRoundTo];
			}
		}

		OutIntegralPart = IntegralPart;
		OutFractionalPart = ValueToRound;
	}
}

void BuildFinalString(const bool bIsNegative, const bool bAlwaysSign, const FDecimalNumberFormattingRules& InFormattingRules, const TCHAR* InIntegralBuffer, const int32 InIntegralLen, const TCHAR* InFractionalBuffer, const int32 InFractionalLen, FString& OutString)
{
	const FDecimalNumberSigningStrings SigningStrings(InFormattingRules, bAlwaysSign ? EDecimalNumberSigningStringsFlags::AlwaysSign : EDecimalNumberSigningStringsFlags::None);

	const FString& FinalPrefixStr = (bIsNegative) ? SigningStrings.GetNegativePrefixString() : SigningStrings.GetPositivePrefixString();
	const FString& FinalSuffixStr = (bIsNegative) ? SigningStrings.GetNegativeSuffixString() : SigningStrings.GetPositiveSuffixString();

	OutString.Reserve(OutString.Len() + FinalPrefixStr.Len() + InIntegralLen + 1 + InFractionalLen + FinalSuffixStr.Len());

	OutString.Append(FinalPrefixStr);
	OutString.AppendChars(InIntegralBuffer, InIntegralLen);
	if (InFractionalLen > 0)
	{
		OutString.AppendChar(InFormattingRules.DecimalSeparatorCharacter);
		OutString.AppendChars(InFractionalBuffer, InFractionalLen);
	}
	OutString.Append(FinalSuffixStr);
}

void IntegralToString(const bool bIsNegative, const uint64 InVal, const FDecimalNumberFormattingRules& InFormattingRules, FNumberFormattingOptions InFormattingOptions, FString& OutString)
{
	SanitizeNumberFormattingOptions(InFormattingOptions);

	// Deal with the integral part (produces a string of the integral part, inserting group separators if requested and required, and padding as needed)
	TCHAR IntegralPartBuffer[MinRequiredIntegralBufferSize];
	const int32 IntegralPartLen = IntegralToString_Common(InVal, InFormattingRules, InFormattingOptions, IntegralPartBuffer, UE_ARRAY_COUNT(IntegralPartBuffer));

	// Deal with any forced fractional part (produces a string zeros up to the required minimum length)
	TCHAR FractionalPartBuffer[MinRequiredIntegralBufferSize];
	int32 FractionalPartLen = 0;
	if (InFormattingOptions.MinimumFractionalDigits > 0)
	{
		const int32 PaddingToApply = FMath::Min(InFormattingOptions.MinimumFractionalDigits, MaxFractionalPrintPrecision);
		for (int32 PaddingIndex = 0; PaddingIndex < PaddingToApply; ++PaddingIndex)
		{
			FractionalPartBuffer[FractionalPartLen++] = InFormattingRules.DigitCharacters[0];
		}
	}
	FractionalPartBuffer[FractionalPartLen] = TEXT('\0');

	BuildFinalString(bIsNegative, InFormattingOptions.AlwaysSign, InFormattingRules, IntegralPartBuffer, IntegralPartLen, FractionalPartBuffer, FractionalPartLen, OutString);
}

FString CultureInvariantDecimalToString(const double InVal, const TCHAR*& InBuffer, const int32 InBufferLen, const FDecimalNumberFormattingRules& InFormattingRules, const FNumberFormattingOptions& InFormattingOptions)
{
	if (!ensure(InBuffer && InBufferLen > 0))
	{
		return FString();
	}

	// Note: Does not consider max digits, this is by design as this method was created to support large floats greater than e18.
	TStringBuilder<128> OutStr;

	bool bUseGrouping = InFormattingOptions.UseGrouping && InFormattingRules.PrimaryGroupingSize > 0;
	uint8 NumIntegralDigits = static_cast<uint8>(FMath::Abs((FMath::LogX(10.0, InVal)))) + 1;
	uint8 NumUntilNextGroup = bUseGrouping ? NumIntegralDigits % InFormattingRules.PrimaryGroupingSize : 0;
	const TCHAR* InBufferEnd = InBuffer + InBufferLen;

	// Apply front padding
	const int32 PaddingToApply = FMath::Max(InFormattingOptions.MinimumIntegralDigits - InBufferLen, 0);
	for (int32 PaddingIndex = 0; PaddingIndex < PaddingToApply; ++PaddingIndex)
	{
		if (bUseGrouping && NumUntilNextGroup-- == 0)
		{
			OutStr += InFormattingRules.GroupingSeparatorCharacter;
			NumUntilNextGroup = InFormattingRules.SecondaryGroupingSize;
		}

		OutStr += InFormattingRules.DigitCharacters[0];
	}

	// Scrape negative, apply at end
	// Skip positive
	bool bIsNegative = false;
	constexpr TCHAR EuropeanNegativePrefix = TEXT('-');
	constexpr TCHAR EuropeanPositivePrefix = TEXT('+');
	if (*InBuffer == EuropeanNegativePrefix)
	{
		bIsNegative = true;
		++InBuffer;
	}
	else if (*InBuffer == EuropeanPositivePrefix)
	{
		++InBuffer;
	}

	// Parse digits & decimal, no grouping on fractional
	bool bParsedFractional = false;
	uint8 FractionalDigitsPrinted = 0;
	while (InBuffer < InBufferEnd)
	{
		constexpr TCHAR EuropeanDecimal = TEXT('.');
		if (*InBuffer == EuropeanDecimal)
		{
			if (InFormattingOptions.MaximumFractionalDigits > 0)
			{
				bParsedFractional = true;
				OutStr += InFormattingRules.DecimalSeparatorCharacter;
				++InBuffer;
				continue;
			}
			break; // Fractional digits not needed; stop parsing the source string now
		}

		if (!bParsedFractional && bUseGrouping && NumUntilNextGroup-- == 0)
		{
			OutStr += InFormattingRules.GroupingSeparatorCharacter;
			NumUntilNextGroup = InFormattingRules.SecondaryGroupingSize - 1; // -1 to account for the digit we're about to print
		}

		// 48 for raw ascii -> int
		int32 CharacterIndex = (int32)InBuffer[0] - 48;
		if (ensureMsgf(CharacterIndex >= 0 && CharacterIndex < UE_ARRAY_COUNT(InFormattingRules.DigitCharacters), TEXT("Found invalid digit character '%c'!"), *InBuffer))
		{
			OutStr += InFormattingRules.DigitCharacters[CharacterIndex];
			FractionalDigitsPrinted += bParsedFractional ? 1 : 0;
			++InBuffer;
		}
		else
		{
			// Invalid digit; stop parsing the source string now
			break;
		}
	}

	// Apply back padding, if back isn't just zero
	double IntegralVal = 0.0;
	double FractionalVal = FMath::Modf(InVal, &IntegralVal);
	if (InFormattingOptions.MaximumFractionalDigits > FractionalDigitsPrinted)
	{
		if (!bParsedFractional)
		{
			OutStr += InFormattingRules.DecimalSeparatorCharacter;
		}

		if (FMath::Abs(FractionalVal) > 0.0)
		{
			const int32 BackPaddingToApply = InFormattingOptions.MaximumFractionalDigits - FractionalDigitsPrinted;
			for (int32 PaddingIndex = 0; PaddingIndex < BackPaddingToApply; ++PaddingIndex)
			{
				OutStr += InFormattingRules.DigitCharacters[0];
			}
		}
	}

	const FDecimalNumberSigningStrings SigningStrings(InFormattingRules, InFormattingOptions.AlwaysSign ? EDecimalNumberSigningStringsFlags::AlwaysSign : EDecimalNumberSigningStringsFlags::None);

	const FString& FinalPrefixStr = (bIsNegative) ? SigningStrings.GetNegativePrefixString() : SigningStrings.GetPositivePrefixString();
	const FString& FinalSuffixStr = (bIsNegative) ? SigningStrings.GetNegativeSuffixString() : SigningStrings.GetPositiveSuffixString();

	OutStr.InsertAt(0, FinalPrefixStr);
	OutStr += FinalSuffixStr;
	
	return FString(OutStr);
}

void FractionalToString(const double InVal, const FDecimalNumberFormattingRules& InFormattingRules, FNumberFormattingOptions InFormattingOptions, FString& OutString)
{
	SanitizeNumberFormattingOptions(InFormattingOptions);

	if (FMath::IsNaN((float)InVal)) //@TODO: FLOATPRECISION: Need a double version?
	{
		OutString.Append(InFormattingRules.NaNString);
		return;
	}

	const bool bIsNegative = FMath::IsNegativeOrNegativeZero(InVal);

	double IntegralPart = 0.0;
	double FractionalPart = 0.0;
	FractionalToString_SplitAndRoundNumber(bIsNegative, InVal, InFormattingOptions.MaximumFractionalDigits, InFormattingOptions.RoundingMode, IntegralPart, FractionalPart);

	// Capture if nearly integer so we can warn users about denorms / near zero values being displayed as zero (Ex: 1e-16)
	const bool bNearlyInteger = bFastDecimalFormatCanIndicateIfNearlyInteger 
		&& InFormattingOptions.IndicateNearlyInteger 
		&& FMath::IsNearlyEqual(IntegralPart, InVal) 
		&& IntegralPart != InVal;

	if (bIsNegative)
	{
		IntegralPart = -IntegralPart;
		FractionalPart = -FractionalPart;
	}

	// Check for float-> int overflow, fallback on regular lex if occurs
	// if fractional part overflows then we are losing precession but the number is still valid
	uint64 IntIntegralPart = static_cast<uint64>(IntegralPart);
	if (IntegralPart - static_cast<double>(IntIntegralPart) > UE_SMALL_NUMBER && bFastDecimalFormatLargeFloatSupport)
	{
		OutString = LexToSanitizedString(InVal);

		// Guard against 'inf', other non-decimal floats.
		if (OutString.IsNumeric())
		{
			const TCHAR* CultureInvariantDecimalBuffer = *OutString;
			OutString = CultureInvariantDecimalToString(InVal, CultureInvariantDecimalBuffer, OutString.Len(), InFormattingRules, InFormattingOptions);
		}
		return;
	}

	// Deal with the integral part (produces a string of the integral part, inserting group separators if requested and required, and padding as needed)
	TCHAR IntegralPartBuffer[MinRequiredIntegralBufferSize];
	const int32 IntegralPartLen = IntegralToString_Common(IntIntegralPart, InFormattingRules, InFormattingOptions, IntegralPartBuffer, UE_ARRAY_COUNT(IntegralPartBuffer));

	// Deal with the fractional part (produces a string of the fractional part, potentially padding with zeros up to InFormattingOptions.MaximumFractionalDigits)
	TCHAR FractionalPartBuffer[MinRequiredIntegralBufferSize];
	int32 FractionalPartLen = 0;
	if (FractionalPart != 0.0)
	{
		FractionalPartLen = IntegralToString_UInt64ToString(static_cast<uint64>(FractionalPart), false, 0, 0, 1, TEXT(' '), InFormattingRules.DigitCharacters, 0, InFormattingOptions.MaximumFractionalDigits, FractionalPartBuffer, UE_ARRAY_COUNT(FractionalPartBuffer));
	
		{
			// Pad the fractional part with any leading zeros that may have been lost when the number was split
			const int32 LeadingZerosToAdd = FMath::Min(InFormattingOptions.MaximumFractionalDigits - FractionalPartLen, MaxFractionalPrintPrecision - FractionalPartLen);
			if (LeadingZerosToAdd > 0)
			{
				FMemory::Memmove(FractionalPartBuffer + LeadingZerosToAdd, FractionalPartBuffer, FractionalPartLen * sizeof(TCHAR));

				for (int32 Index = 0; Index < LeadingZerosToAdd; ++Index)
				{
					FractionalPartBuffer[Index] = InFormattingRules.DigitCharacters[0];
				}

				FractionalPartLen += LeadingZerosToAdd;
			}
		}

		// Trim any trailing zeros back down to InFormattingOptions.MinimumFractionalDigits
		while (FractionalPartLen > InFormattingOptions.MinimumFractionalDigits && FractionalPartBuffer[FractionalPartLen - 1] == InFormattingRules.DigitCharacters[0])
		{
			--FractionalPartLen;
		}
	}
	FractionalPartBuffer[FractionalPartLen] = TCHAR('\0');

	// Pad the fractional part with any zeros that may have been missed so far
	{
		const int32 PaddingToApply = FMath::Min(InFormattingOptions.MinimumFractionalDigits - FractionalPartLen, MaxFractionalPrintPrecision - FractionalPartLen);
		for (int32 PaddingIndex = 0; PaddingIndex < PaddingToApply; ++PaddingIndex)
		{
			FractionalPartBuffer[FractionalPartLen++] = InFormattingRules.DigitCharacters[0];
		}
		FractionalPartBuffer[FractionalPartLen] = TCHAR('\0');
	}

	// Append an '...' if the value is nearly an integer
	if (bNearlyInteger && FractionalPartLen + 3 < UE_ARRAY_COUNT(FractionalPartBuffer))
	{
		FractionalPartBuffer[FractionalPartLen++] = '.';
		FractionalPartBuffer[FractionalPartLen++] = '.';
		FractionalPartBuffer[FractionalPartLen++] = '.';
		FractionalPartBuffer[FractionalPartLen] = TCHAR('\0');
	}

	BuildFinalString(bIsNegative, InFormattingOptions.AlwaysSign, InFormattingRules, IntegralPartBuffer, IntegralPartLen, FractionalPartBuffer, FractionalPartLen, OutString);
}

enum class EDecimalNumberParseFlags : uint8
{
	None = 0,
	AllowLeadingSign = 1<<0,
	AllowTrailingSign = 1<<1,
	AllowDecimalSeparators = 1<<2,
	AllowGroupSeparators = 1<<3,
	TestLimits = 1<<4,
	ClampValue = 1<<5,
};
ENUM_CLASS_FLAGS(EDecimalNumberParseFlags);

struct FDecimalNumberSignParser
{
public:
	explicit FDecimalNumberSignParser(const FDecimalNumberFormattingRules& InFormattingRules)
		: Localized_DefaultSigned(InFormattingRules, EDecimalNumberSigningStringsFlags::None)
		, Localized_AlwaysSigned(InFormattingRules, EDecimalNumberSigningStringsFlags::AlwaysSign)
		, ASCII_DefaultSigned(InFormattingRules, EDecimalNumberSigningStringsFlags::UseASCIISigns)
		, ASCII_AlwaysSigned(InFormattingRules, EDecimalNumberSigningStringsFlags::AlwaysSign | EDecimalNumberSigningStringsFlags::UseASCIISigns)
	{
	}

	bool ParseLeadingSign(const TCHAR*& InBuffer, bool& OutIsNegative) const
	{
		return ParseSigningStringImpl(InBuffer, OutIsNegative, Localized_DefaultSigned.GetPositivePrefixString(), false)
			|| ParseSigningStringImpl(InBuffer, OutIsNegative, Localized_DefaultSigned.GetNegativePrefixString(), true)
			|| ParseSigningStringImpl(InBuffer, OutIsNegative, Localized_AlwaysSigned.GetPositivePrefixString(), false)
			|| ParseSigningStringImpl(InBuffer, OutIsNegative, Localized_AlwaysSigned.GetNegativePrefixString(), true)
			|| ParseSigningStringImpl(InBuffer, OutIsNegative, ASCII_DefaultSigned.GetPositivePrefixString(), false)
			|| ParseSigningStringImpl(InBuffer, OutIsNegative, ASCII_DefaultSigned.GetNegativePrefixString(), true)
			|| ParseSigningStringImpl(InBuffer, OutIsNegative, ASCII_AlwaysSigned.GetPositivePrefixString(), false)
			|| ParseSigningStringImpl(InBuffer, OutIsNegative, ASCII_AlwaysSigned.GetNegativePrefixString(), true);
	}

	bool ParseTrailingSign(const TCHAR*& InBuffer, bool& OutIsNegative) const
	{
		return ParseSigningStringImpl(InBuffer, OutIsNegative, Localized_DefaultSigned.GetPositiveSuffixString(), false)
			|| ParseSigningStringImpl(InBuffer, OutIsNegative, Localized_DefaultSigned.GetNegativeSuffixString(), true)
			|| ParseSigningStringImpl(InBuffer, OutIsNegative, Localized_AlwaysSigned.GetPositiveSuffixString(), false)
			|| ParseSigningStringImpl(InBuffer, OutIsNegative, Localized_AlwaysSigned.GetNegativeSuffixString(), true)
			|| ParseSigningStringImpl(InBuffer, OutIsNegative, ASCII_DefaultSigned.GetPositiveSuffixString(), false)
			|| ParseSigningStringImpl(InBuffer, OutIsNegative, ASCII_DefaultSigned.GetNegativeSuffixString(), true)
			|| ParseSigningStringImpl(InBuffer, OutIsNegative, ASCII_AlwaysSigned.GetPositiveSuffixString(), false)
			|| ParseSigningStringImpl(InBuffer, OutIsNegative, ASCII_AlwaysSigned.GetNegativeSuffixString(), true);
	}

private:
	static bool ParseSigningStringImpl(const TCHAR*& InBuffer, bool& OutIsNegative, const FString& InSigningString, const bool bSigningStringIsNegative)
	{
		if (InSigningString.Len() > 0 && FCString::Strncmp(InBuffer, *InSigningString, InSigningString.Len()) == 0)
		{
			OutIsNegative |= bSigningStringIsNegative;
			InBuffer += InSigningString.Len();
			return true;
		}
		return false;
	}

	const FDecimalNumberSigningStrings Localized_DefaultSigned;
	const FDecimalNumberSigningStrings Localized_AlwaysSigned;
	const FDecimalNumberSigningStrings ASCII_DefaultSigned;
	const FDecimalNumberSigningStrings ASCII_AlwaysSigned;
};

bool StringToIntegral_StringToUInt64(const TCHAR*& InBuffer, const TCHAR* InBufferEnd, const FDecimalNumberFormattingRules& InFormattingRules, const FDecimalNumberSignParser& InSignParser, const EDecimalNumberParseFlags& InParseFlags, const int32 InMaxDigitsToParse, bool& OutIsNegative, bool& OutIsOverflow, uint64& OutVal, uint8& OutDigitCount)
{
	OutIsNegative = false;
	OutIsOverflow = false;
	OutVal = 0;
	OutDigitCount = 0;

	// Empty string?
	if (*InBuffer == 0)
	{
		return true;
	}

	// Parse the leading sign (if present)
	if (InSignParser.ParseLeadingSign(InBuffer, OutIsNegative) && !EnumHasAnyFlags(InParseFlags, EDecimalNumberParseFlags::AllowLeadingSign))
	{
		return false;
	}

	const bool bTestForOverflow = EnumHasAnyFlags(InParseFlags, EDecimalNumberParseFlags::TestLimits | EDecimalNumberParseFlags::ClampValue);

	// Parse the number, stopping once we find the end of the string or a decimal separator
	static const TCHAR EuropeanNumerals[] = { TEXT('0'), TEXT('1'), TEXT('2'), TEXT('3'), TEXT('4'), TEXT('5'), TEXT('6'), TEXT('7'), TEXT('8'), TEXT('9') };
	bool bFoundUnexpectedNonNumericCharacter = false;
	while (InBuffer < InBufferEnd && *InBuffer != InFormattingRules.DecimalSeparatorCharacter)
	{
		// Skip group separators
		if (*InBuffer == InFormattingRules.GroupingSeparatorCharacter)
		{
			if (!EnumHasAnyFlags(InParseFlags, EDecimalNumberParseFlags::AllowGroupSeparators))
			{
				return false;
			}
			++InBuffer;
			continue;
		}

		// Process numeric characters (also test European numerals in case they were used by a language that doesn't normally use them)
		bool bValidChar = false;
		for (int32 CharIndex = 0; CharIndex < UE_ARRAY_COUNT(InFormattingRules.DigitCharacters); ++CharIndex)
		{
			if ((*InBuffer == InFormattingRules.DigitCharacters[CharIndex]) || (*InBuffer == EuropeanNumerals[CharIndex]))
			{
				++InBuffer;
				if (OutDigitCount < InMaxDigitsToParse)
				{
					++OutDigitCount;
					uint64 NewVal = (OutVal * 10) + CharIndex;
					if (bTestForOverflow && NewVal <= OutVal && OutVal != 0)
					{
						OutIsOverflow = true;
						if (EnumHasAnyFlags(InParseFlags, EDecimalNumberParseFlags::TestLimits))
						{
							return false;
						}
					}
					OutVal = NewVal;
				}
				else if (bTestForOverflow)
				{
					// Found a number too big to be represented
					OutIsOverflow = true;
					if (EnumHasAnyFlags(InParseFlags, EDecimalNumberParseFlags::TestLimits))
					{
						return false;
					}
				}
				bValidChar = true;
				break;
			}
		}

		// Found an non-numeric character?
		if (!bValidChar)
		{
			bFoundUnexpectedNonNumericCharacter = true;
			break;
		}
	}

	// Walk over the decimal separator
	if (*InBuffer == InFormattingRules.DecimalSeparatorCharacter)
	{
		if (!EnumHasAnyFlags(InParseFlags, EDecimalNumberParseFlags::AllowDecimalSeparators))
		{
			return false;
		}
		++InBuffer;
	}

	// Parse the trailing sign (if present)
	if (InSignParser.ParseTrailingSign(InBuffer, OutIsNegative))
	{
		// The unexpected character was the trailing sign - clear that flag
		bFoundUnexpectedNonNumericCharacter = false;

		if (!EnumHasAnyFlags(InParseFlags, EDecimalNumberParseFlags::AllowTrailingSign))
		{
			return false;
		}
	}

	return !bFoundUnexpectedNonNumericCharacter;
}

FORCEINLINE bool StringToIntegral_Common(const TCHAR*& InBuffer, const TCHAR* InBufferEnd, const FDecimalNumberFormattingRules& InFormattingRules, const FNumberParsingOptions& InParsingOptions, const FDecimalNumberSignParser& InSignParser, bool& OutIsNegative, bool& OutIsOverflow, uint64& OutVal, uint8& OutDigitCount)
{
	return StringToIntegral_StringToUInt64(
		InBuffer, 
		InBufferEnd, 
		InFormattingRules, 
		InSignParser, 
		EDecimalNumberParseFlags::AllowLeadingSign | EDecimalNumberParseFlags::AllowTrailingSign | EDecimalNumberParseFlags::AllowDecimalSeparators
			| (InParsingOptions.UseGrouping ? EDecimalNumberParseFlags::AllowGroupSeparators : EDecimalNumberParseFlags::None)
			| (InParsingOptions.InsideLimits ? EDecimalNumberParseFlags::TestLimits : EDecimalNumberParseFlags::None)
			| (InParsingOptions.UseClamping ? EDecimalNumberParseFlags::ClampValue : EDecimalNumberParseFlags::None),
		MaxIntegralPrintLength,
		OutIsNegative,
		OutIsOverflow,
		OutVal, 
		OutDigitCount
		);
}

bool StringToIntegral(const TCHAR* InStr, const int32 InStrLen, const FDecimalNumberFormattingRules& InFormattingRules, const FNumberParsingOptions& InParsingOptions, const FDecimalNumberIntegralLimits& InLimits, bool& OutIsNegative, uint64& OutVal, int32* OutParsedLen)
{
	const TCHAR* Buffer = InStr;
	const TCHAR* BufferEnd = InStr + InStrLen;
	const FDecimalNumberSignParser SignParser(InFormattingRules);

	// Parse the integral part of the number
	bool bIsOverflow = false;
	uint8 IntegralPartDigitCount = 0;
	bool bResult = StringToIntegral_Common(Buffer, BufferEnd, InFormattingRules, InParsingOptions, SignParser, OutIsNegative, bIsOverflow, OutVal, IntegralPartDigitCount);

	// A number can only be valid if we actually parsed some digits
	bResult &= IntegralPartDigitCount > 0;

	if (bResult && InParsingOptions.InsideLimits)
	{
		bResult &= !bIsOverflow;
		if (InLimits.bIsNumericSigned)
		{
			uint64 NegativeMinLimit = OutIsNegative ? InLimits.NumericLimitLowest * -1 : InLimits.NumericLimitMax;	//ie. -128 * -1 == 128 | 127
			bResult &= OutVal <= NegativeMinLimit;
		}
		else
		{
			bResult &= !OutIsNegative;
			bResult &= OutVal <= InLimits.NumericLimitMax;
		}
	}

	if (bResult && InParsingOptions.UseClamping)
	{
		if (bIsOverflow)
		{
			OutVal = OutIsNegative ? uint64(InLimits.NumericLimitLowest * -1) : InLimits.NumericLimitMax;
		}
		else
		{
			OutVal = FMath::Clamp<uint64>(OutVal, 0, OutIsNegative ? (InLimits.NumericLimitLowest * -1) : InLimits.NumericLimitMax);
		}
	}

	// Only fill in the length if we actually parsed some digits
	if (IntegralPartDigitCount > 0 && OutParsedLen)
	{
		*OutParsedLen = UE_PTRDIFF_TO_INT32(Buffer - InStr);
	}

	return bResult;
}

bool StringToCultureInvariantDecimal(const TCHAR*& InBuffer, const TCHAR* InBufferEnd, const FDecimalNumberFormattingRules& InFormattingRules, const FNumberParsingOptions& InParsingOptions, const FDecimalNumberSignParser& InSignParser, TStringBuilder<128>& OutInvariantDecimal)
{
	// Empty string?
	if (*InBuffer == 0)
	{
		return true;
	}

	EDecimalNumberParseFlags InParseFlags = EDecimalNumberParseFlags::AllowLeadingSign | EDecimalNumberParseFlags::AllowTrailingSign | EDecimalNumberParseFlags::AllowDecimalSeparators
		| (InParsingOptions.UseGrouping ? EDecimalNumberParseFlags::AllowGroupSeparators : EDecimalNumberParseFlags::None)
		| (InParsingOptions.InsideLimits ? EDecimalNumberParseFlags::TestLimits : EDecimalNumberParseFlags::None)
		| (InParsingOptions.UseClamping ? EDecimalNumberParseFlags::ClampValue : EDecimalNumberParseFlags::None);

	// Parse the leading sign (if present)
	bool bIsNegative = false;
	InSignParser.ParseLeadingSign(InBuffer, bIsNegative);

	static const TCHAR InvariantNegativePrefix = TEXT('-');
	if (bIsNegative)
	{
		OutInvariantDecimal += InvariantNegativePrefix;
	}

	// Parse the number, stopping once we find the end of the string or a decimal separator
	static const TCHAR EuropeanNumerals[] = { TEXT('0'), TEXT('1'), TEXT('2'), TEXT('3'), TEXT('4'), TEXT('5'), TEXT('6'), TEXT('7'), TEXT('8'), TEXT('9') };
	bool bFoundUnexpectedNonNumericCharacter = false;
	while (InBuffer < InBufferEnd)
	{
		// Skip group separators
		if (*InBuffer == InFormattingRules.GroupingSeparatorCharacter)
		{
			if (!EnumHasAnyFlags(InParseFlags, EDecimalNumberParseFlags::AllowGroupSeparators))
			{
				return false;
			}
			++InBuffer;
			continue;
		}

		// Walk over the decimal separator
		static const TCHAR InvariantDecimal = TEXT('.');
		if (*InBuffer == InFormattingRules.DecimalSeparatorCharacter)
		{
			if (!EnumHasAnyFlags(InParseFlags, EDecimalNumberParseFlags::AllowDecimalSeparators))
			{
				return false;
			}
			++InBuffer;
			OutInvariantDecimal += InvariantDecimal;
			continue;
		}

		// Process numeric characters (also test European numerals in case they were used by a language that doesn't normally use them)
		bool bValidChar = false;
		for (int32 CharIndex = 0; CharIndex < UE_ARRAY_COUNT(InFormattingRules.DigitCharacters); ++CharIndex)
		{
			if ((*InBuffer == InFormattingRules.DigitCharacters[CharIndex]) || (*InBuffer == EuropeanNumerals[CharIndex]))
			{
				// We don't consider MaxIntegralPrintLength, since this method is used to deal with large string to string values
				++InBuffer;
				OutInvariantDecimal += EuropeanNumerals[CharIndex];
				bValidChar = true;
				break;
			}
		}

		// Found an non-numeric character?
		if (!bValidChar)
		{
			bFoundUnexpectedNonNumericCharacter = true;
			break;
		}
	}

	// Parse the trailing sign (if present)
	if (InSignParser.ParseTrailingSign(InBuffer, bIsNegative))
	{
		// The unexpected character was the trailing sign - clear that flag
		bFoundUnexpectedNonNumericCharacter = false;
	}

	if (OutInvariantDecimal.LastChar() != '\0')
	{
		OutInvariantDecimal += '\0';
	}

	return !bFoundUnexpectedNonNumericCharacter;
}

bool StringToFractional(const TCHAR* InStr, const int32 InStrLen, const FDecimalNumberFormattingRules& InFormattingRules, const FNumberParsingOptions& InParsingOptions, const FDecimalNumberFractionalLimits& InLimits, double& OutVal, int32* OutParsedLen)
{
	const TCHAR* Buffer = InStr;
	const TCHAR* BufferEnd = InStr + InStrLen;
	const FDecimalNumberSignParser SignParser(InFormattingRules);

	// Parse the integral part of the number, if this succeeds then Buffer will be pointing at the first digit past the decimal separator
	bool bIntegralPartIsNegative = false;
	bool bIntegralPartIsOverflow = false;
	uint64 IntegralPart = 0;
	uint8 IntegralPartDigitCount = 0;
	bool bResult = StringToIntegral_Common(Buffer, BufferEnd, InFormattingRules, InParsingOptions, SignParser, bIntegralPartIsNegative, bIntegralPartIsOverflow, IntegralPart, IntegralPartDigitCount);
	bResult &= !bIntegralPartIsOverflow;

	if (bIntegralPartIsOverflow && bFastDecimalFormatLargeFloatSupport)
	{
		const TCHAR* InvariantBuffer = InStr;
		TStringBuilder<128> InvariantDecimal;
		if (StringToCultureInvariantDecimal(InvariantBuffer, BufferEnd, InFormattingRules, InParsingOptions, SignParser, InvariantDecimal))
		{
			LexFromString(OutVal, InvariantDecimal.GetData());

			// We overflowed, so have callers act as if string length hasn't changed
			*OutParsedLen = InStrLen;
			return true;
		}
	}

	// Parse the fractional part of the number
	bool bFractionPartIsNegative = false;
	bool bFractionPartIsOverflow = false;
	uint64 FractionalPart = 0;
	uint8 FractionalPartDigitCount = 0;
	if (bResult && Buffer > InStr && *(Buffer - 1) == InFormattingRules.DecimalSeparatorCharacter)
	{
		// Only parse the fractional part of the number if the preceding character was a decimal separator
		bResult &= StringToIntegral_StringToUInt64(Buffer, BufferEnd, InFormattingRules, SignParser, EDecimalNumberParseFlags::AllowTrailingSign, MaxFractionalPrintPrecision, bFractionPartIsNegative, bFractionPartIsOverflow, FractionalPart, FractionalPartDigitCount);
		// if bFractionPartIsOverflow then we are losing precession but the number is still valid (and should be bellow MaxFractionalPrintPrecision)
	}

	// A number can only be valid if we actually parsed some digits
	const uint8 TotalDigitCount = IntegralPartDigitCount + FractionalPartDigitCount;
	bResult &= TotalDigitCount > 0;

	// Build the final number
	OutVal = static_cast<double>(IntegralPart);
	OutVal += (static_cast<double>(FractionalPart) / (double)Pow10Table[FractionalPartDigitCount]);
	OutVal *= ((bIntegralPartIsNegative || bFractionPartIsNegative) ? -1.0 : 1.0);
	

	if (bResult && InParsingOptions.InsideLimits)
	{
		bResult &= OutVal >= InLimits.NumericLimitLowest && OutVal <= InLimits.NumericLimitMax;
	}

	if (bResult && InParsingOptions.UseClamping)
	{
		OutVal = FMath::Clamp(OutVal, InLimits.NumericLimitLowest, InLimits.NumericLimitMax);
	}

	// Only fill in the length if we actually parsed some digits
	if (TotalDigitCount > 0 && OutParsedLen)
	{
		*OutParsedLen = UE_PTRDIFF_TO_INT32(Buffer - InStr);
	}

	return bResult;
}

} // namespace Internal

const FDecimalNumberFormattingRules& GetCultureAgnosticFormattingRules()
{
	auto BuildAgnosticFormattingRules = []()
	{
		FDecimalNumberFormattingRules AgnosticFormattingRules;
		AgnosticFormattingRules.NaNString = TEXT("NaN");
		AgnosticFormattingRules.NegativePrefixString = TEXT("-");
		AgnosticFormattingRules.PlusString = TEXT("+");
		AgnosticFormattingRules.MinusString = TEXT("-");
		AgnosticFormattingRules.GroupingSeparatorCharacter = TEXT(',');
		AgnosticFormattingRules.DecimalSeparatorCharacter = TEXT('.');
		AgnosticFormattingRules.PrimaryGroupingSize = 3;
		AgnosticFormattingRules.SecondaryGroupingSize = 3;
		return AgnosticFormattingRules;
	};

	static const FDecimalNumberFormattingRules CultureAgnosticFormattingRules = BuildAgnosticFormattingRules();
	return CultureAgnosticFormattingRules;
}

uint64 Pow10(const int32 InExponent)
{
	const int32 ClampedExponent = FMath::Min(InExponent, Internal::MaxFractionalPrintPrecision);
	return Internal::Pow10Table[ClampedExponent];
}

} // namespace FastDecimalFormat
