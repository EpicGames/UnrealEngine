// Copyright Epic Games, Inc. All Rights Reserved.

#include "Internationalization/GatherableTextData.h"

#include "CoreTypes.h"
#include "Serialization/StructuredArchive.h"
#include "Serialization/StructuredArchiveAdapters.h"
#include "Serialization/StructuredArchiveNameHelpers.h"
#include "Serialization/StructuredArchiveSlots.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

class FArchive;

void FTextSourceSiteContext::StreamArchive(FArchive& Archive)
{
	FStructuredArchiveFromArchive(Archive).GetSlot() << *this;
}

void FTextSourceSiteContext::StreamStructuredArchive(FStructuredArchive::FSlot Slot)
{
	FStructuredArchive::FRecord Record = Slot.EnterRecord();
	Record << SA_VALUE(TEXT("KeyName"), KeyName);
	Record << SA_VALUE(TEXT("SiteDescription"), SiteDescription);
	Record << SA_VALUE(TEXT("IsEditorOnly"), IsEditorOnly);
	Record << SA_VALUE(TEXT("IsOptional"), IsOptional);
#if WITH_EDITORONLY_DATA
	FArchive& UnderlyingArchive = Slot.GetUnderlyingArchive();
	UnderlyingArchive.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	if (UnderlyingArchive.CustomVer(FFortniteMainBranchObjectVersion::GUID) >= FFortniteMainBranchObjectVersion::AddDevNotesToFText && !UnderlyingArchive.IsFilterEditorOnly())
	{
		Record << SA_VALUE(TEXT("DevNotes"), DevNotes);
	}
#endif
	Record << SA_VALUE(TEXT("InfoMetaData"), InfoMetaData);
	Record << SA_VALUE(TEXT("KeyMetaData"), KeyMetaData);
}

void FTextSourceData::StreamArchive(FArchive& Archive)
{
	FStructuredArchiveFromArchive(Archive).GetSlot() << *this;
}

void FTextSourceData::StreamStructuredArchive(FStructuredArchive::FSlot Slot)
{
	FStructuredArchive::FRecord Record = Slot.EnterRecord();
	Record << SA_VALUE(TEXT("SourceString"), SourceString);
	Record << SA_VALUE(TEXT("SourceStringMetaData"), SourceStringMetaData);
}

void FGatherableTextData::StreamArchive(FArchive& Archive)
{
	FStructuredArchiveFromArchive(Archive).GetSlot() << *this;
}

void FGatherableTextData::StreamStructuredArchive(FStructuredArchive::FSlot Slot)
{
	FStructuredArchive::FRecord Record = Slot.EnterRecord();
	Record << SA_VALUE(TEXT("NamespaceName"), NamespaceName);
	Record << SA_VALUE(TEXT("SourceData"), SourceData);
	Record << SA_VALUE(TEXT("SourceSiteContexts"), SourceSiteContexts);
}