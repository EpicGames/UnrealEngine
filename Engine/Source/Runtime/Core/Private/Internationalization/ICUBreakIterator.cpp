// Copyright Epic Games, Inc. All Rights Reserved.

#include "Internationalization/ICUBreakIterator.h"
#include "Misc/ScopeLock.h"
#include "Internationalization/Internationalization.h"

#if UE_ENABLE_ICU
#include "Internationalization/ICUTextCharacterIterator.h"
#include "Internationalization/ICUCulture.h"

FICUBreakIteratorManager* FICUBreakIteratorManager::Singleton = nullptr;

void FICUBreakIteratorManager::Create()
{
	check(!Singleton);
	Singleton = new FICUBreakIteratorManager();
}

void FICUBreakIteratorManager::Destroy()
{
	check(Singleton);
	delete Singleton;
	Singleton = nullptr;
}

bool FICUBreakIteratorManager::IsInitialized()
{
	return Singleton != nullptr;
}

FICUBreakIteratorManager& FICUBreakIteratorManager::Get()
{
	check(Singleton);
	return *Singleton;
}

TWeakPtr<icu::BreakIterator> FICUBreakIteratorManager::CreateCharacterBoundaryIterator()
{
    FScopeLock ScopeLock(&AllocatedIteratorsCS);
	TSharedRef<icu::BreakIterator> Iterator = MakeShareable( FInternationalization::Get().GetDefaultCulture()->Implementation->GetBreakIterator(EBreakIteratorType::Grapheme)->clone() );
	{
		AllocatedIterators.Add(Iterator);
	}
	return Iterator;
}

TWeakPtr<icu::BreakIterator> FICUBreakIteratorManager::CreateWordBreakIterator()
{
    FScopeLock ScopeLock(&AllocatedIteratorsCS);
	TSharedRef<icu::BreakIterator> Iterator = MakeShareable( FInternationalization::Get().GetDefaultCulture()->Implementation->GetBreakIterator(EBreakIteratorType::Word)->clone() );
	{
		AllocatedIterators.Add(Iterator);
	}
	return Iterator;
}

TWeakPtr<icu::BreakIterator> FICUBreakIteratorManager::CreateLineBreakIterator()
{
    FScopeLock ScopeLock(&AllocatedIteratorsCS);
	TSharedRef<icu::BreakIterator> Iterator = MakeShareable( FInternationalization::Get().GetDefaultCulture()->Implementation->GetBreakIterator(EBreakIteratorType::Line)->clone() );
	{
		AllocatedIterators.Add(Iterator);
	}
	return Iterator;
}

void FICUBreakIteratorManager::DestroyIterator(TWeakPtr<icu::BreakIterator>& InIterator)
{
    FScopeLock ScopeLock(&AllocatedIteratorsCS);
	TSharedPtr<icu::BreakIterator> Iterator = InIterator.Pin();
	if(Iterator.IsValid())
	{
		AllocatedIterators.Remove(Iterator);
	}
	InIterator.Reset();
}

FICUBreakIterator::FICUBreakIterator(TWeakPtr<icu::BreakIterator>&& InICUBreakIteratorHandle)
	: ICUBreakIteratorHandle(InICUBreakIteratorHandle)
{
}

FICUBreakIterator::~FICUBreakIterator()
{
	if (FICUBreakIteratorManager::IsInitialized())
	{
		// This assumes that FICUBreakIterator owns the iterator, and that nothing ever copies an FICUBreakIterator instance
		FICUBreakIteratorManager::Get().DestroyIterator(ICUBreakIteratorHandle);
	}
}

void FICUBreakIterator::SetString(FString&& InString)
{
	GetInternalBreakIterator()->adoptText(new FICUTextCharacterIterator(MoveTemp(InString))); // ICUBreakIterator takes ownership of this instance
	ResetToBeginning();
}

void FICUBreakIterator::SetStringRef(FStringView InString)
{
	GetInternalBreakIterator()->adoptText(new FICUTextCharacterIterator(InString)); // ICUBreakIterator takes ownership of this instance
	ResetToBeginning();
}

int32 FICUBreakIterator::GetCurrentPosition() const
{
	TSharedRef<icu::BreakIterator> BrkIt = GetInternalBreakIterator();
	const int32 InternalIndex = BrkIt->current();
	return static_cast<FICUTextCharacterIterator&>(BrkIt->getText()).InternalIndexToSourceIndex(InternalIndex);
}

int32 FICUBreakIterator::ResetToBeginning()
{
	TSharedRef<icu::BreakIterator> BrkIt = GetInternalBreakIterator();
	const int32 InternalIndex = BrkIt->first();
	return static_cast<FICUTextCharacterIterator&>(BrkIt->getText()).InternalIndexToSourceIndex(InternalIndex);
}

int32 FICUBreakIterator::ResetToEnd()
{
	TSharedRef<icu::BreakIterator> BrkIt = GetInternalBreakIterator();
	const int32 InternalIndex = BrkIt->last();
	return static_cast<FICUTextCharacterIterator&>(BrkIt->getText()).InternalIndexToSourceIndex(InternalIndex);
}

int32 FICUBreakIterator::MoveToPrevious()
{
	TSharedRef<icu::BreakIterator> BrkIt = GetInternalBreakIterator();
	const int32 InternalIndex = BrkIt->previous();
	return static_cast<FICUTextCharacterIterator&>(BrkIt->getText()).InternalIndexToSourceIndex(InternalIndex);
}

int32 FICUBreakIterator::MoveToNext()
{
	TSharedRef<icu::BreakIterator> BrkIt = GetInternalBreakIterator();
	const int32 InternalIndex = BrkIt->next();
	return static_cast<FICUTextCharacterIterator&>(BrkIt->getText()).InternalIndexToSourceIndex(InternalIndex);
}

int32 FICUBreakIterator::MoveToCandidateBefore(const int32 InIndex)
{
	TSharedRef<icu::BreakIterator> BrkIt = GetInternalBreakIterator();
	const int32 InitialInternalIndex = static_cast<FICUTextCharacterIterator&>(BrkIt->getText()).SourceIndexToInternalIndex(InIndex);
	const int32 InternalIndex = BrkIt->preceding(InitialInternalIndex);
	return static_cast<FICUTextCharacterIterator&>(BrkIt->getText()).InternalIndexToSourceIndex(InternalIndex);
}

int32 FICUBreakIterator::MoveToCandidateAfter(const int32 InIndex)
{
	TSharedRef<icu::BreakIterator> BrkIt = GetInternalBreakIterator();
	const int32 InitialInternalIndex = static_cast<FICUTextCharacterIterator&>(BrkIt->getText()).SourceIndexToInternalIndex(InIndex);
	const int32 InternalIndex = BrkIt->following(InitialInternalIndex);
	return static_cast<FICUTextCharacterIterator&>(BrkIt->getText()).InternalIndexToSourceIndex(InternalIndex);
}

TSharedRef<icu::BreakIterator> FICUBreakIterator::GetInternalBreakIterator() const
{
	return ICUBreakIteratorHandle.Pin().ToSharedRef();
}

#endif
