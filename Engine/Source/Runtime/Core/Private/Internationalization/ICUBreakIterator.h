// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Set.h"
#include "Templates/SharedPointer.h"
#include "Internationalization/IBreakIterator.h"

#if UE_ENABLE_ICU

THIRD_PARTY_INCLUDES_START
	#include <unicode/brkiter.h>
THIRD_PARTY_INCLUDES_END

/**
 * Manages the lifespan of ICU break iterators
 */
class FICUBreakIteratorManager
{
public:
	static void Create();
	static void Destroy();
	static bool IsInitialized();
	static FICUBreakIteratorManager& Get();

	TWeakPtr<icu::BreakIterator> CreateCharacterBoundaryIterator();
	TWeakPtr<icu::BreakIterator> CreateWordBreakIterator();
	TWeakPtr<icu::BreakIterator> CreateLineBreakIterator();
	void DestroyIterator(TWeakPtr<icu::BreakIterator>& InIterator);

private:
	static FICUBreakIteratorManager* Singleton;
	FCriticalSection AllocatedIteratorsCS;
	TSet<TSharedPtr<icu::BreakIterator>> AllocatedIterators;
};

/**
 * Wraps an ICU break iterator instance inside our own break iterator API
 */
class FICUBreakIterator : public IBreakIterator
{
public:
	FICUBreakIterator(TWeakPtr<icu::BreakIterator>&& InICUBreakIteratorHandle);
	virtual ~FICUBreakIterator();

	virtual void SetString(FString&& InString) override;
	virtual void SetStringRef(FStringView InString) override;

	virtual int32 GetCurrentPosition() const override;

	virtual int32 ResetToBeginning() override;
	virtual int32 ResetToEnd() override;

	virtual int32 MoveToPrevious() override;
	virtual int32 MoveToNext() override;
	virtual int32 MoveToCandidateBefore(const int32 InIndex) override;
	virtual int32 MoveToCandidateAfter(const int32 InIndex) override;

protected:
	TSharedRef<icu::BreakIterator> GetInternalBreakIterator() const;

private:
	TWeakPtr<icu::BreakIterator> ICUBreakIteratorHandle;
};

#endif
