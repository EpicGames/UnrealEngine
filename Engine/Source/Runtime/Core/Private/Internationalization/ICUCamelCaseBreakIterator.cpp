// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "Templates/SharedPointer.h"
#include "Internationalization/IBreakIterator.h"
#include "Internationalization/BreakIterator.h"
#include "Internationalization/CamelCaseBreakIterator.h"

#if UE_ENABLE_ICU
#include "Internationalization/ICUTextCharacterIterator.h"
THIRD_PARTY_INCLUDES_START
	#include <unicode/uchar.h>
THIRD_PARTY_INCLUDES_END

class FICUCamelCaseBreakIterator : public FCamelCaseBreakIterator
{
protected:
	virtual void TokenizeString(FTokensArray& OutTokens) override;
};

void FICUCamelCaseBreakIterator::TokenizeString(FTokensArray& OutTokens)
{
	OutTokens.Empty(String.Len());

	if (String.Len() > 0)
	{
		FICUTextCharacterIterator CharIter(String);
		for (CharIter.setToStart(); CharIter.current32() != FICUTextCharacterIterator::DONE; CharIter.next32PostInc())
		{
			const UChar32 CurrentChar = CharIter.current32();

			ETokenType TokenType = ETokenType::Other;
			if (u_isULowercase(CurrentChar))
			{
				TokenType = ETokenType::Lowercase;
			}
			else if (u_isUUppercase(CurrentChar))
			{
				TokenType = ETokenType::Uppercase;
			}
			else if (u_isdigit(CurrentChar))
			{
				TokenType = ETokenType::Digit;
			}

			const int32 CharIndex = CharIter.InternalIndexToSourceIndex(CharIter.getIndex());
			OutTokens.Emplace(FToken(TokenType, CharIndex));
		}
	}

	OutTokens.Emplace(FToken(ETokenType::Null, String.Len()));

	// There should always be at least one token for the end of the string
	check(OutTokens.Num());
}

TSharedRef<IBreakIterator> FBreakIterator::CreateCamelCaseBreakIterator()
{
	return MakeShareable(new FICUCamelCaseBreakIterator());
}

#endif
