// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "Templates/SharedPointer.h"
#include "Internationalization/IBreakIterator.h"
#include "Internationalization/BreakIterator.h"

#if UE_ENABLE_ICU
#include "Internationalization/ICUBreakIterator.h"

TSharedRef<IBreakIterator> FBreakIterator::CreateCharacterBoundaryIterator()
{
	return MakeShareable(new FICUBreakIterator(FICUBreakIteratorManager::Get().CreateCharacterBoundaryIterator()));
}

TSharedRef<IBreakIterator> FBreakIterator::GetCharacterBoundaryIterator()
{
	static thread_local TSharedPtr<IBreakIterator> CachedCharacterBoundaryIterator;
	if (!CachedCharacterBoundaryIterator.IsValid())
	{
		CachedCharacterBoundaryIterator = MakeShareable(new FICUBreakIterator(FICUBreakIteratorManager::Get().CreateCharacterBoundaryIterator()));
	}

	check(CachedCharacterBoundaryIterator.IsValid());
	return CachedCharacterBoundaryIterator.ToSharedRef();
}

#endif
