// Copyright Epic Games, Inc. All Rights Reserved.

#include "Internationalization/ICUCulture.h"
#include "Internationalization/Internationalization.h"
#include "Misc/ScopeLock.h"
#include "Containers/SortedMap.h"
#include "Internationalization/FastDecimalFormat.h"

#if UE_ENABLE_ICU

DEFINE_LOG_CATEGORY_STATIC(LogICUCulture, Log, All);

#include "Internationalization/ICUUtilities.h"
#include "HAL/IConsoleManager.h"

static TAutoConsoleVariable<bool> CVarUseLocaleSpecificDigitCharacters(
	TEXT("Localization.UseLocaleSpecificDigitCharacters"),
	true,
	TEXT("False: Locales will always use Arabic digit characters (eg, 1234), True: Locales will use the digit characters specified in their CLDR data (default)."),
	ECVF_Default
	);

static TAutoConsoleVariable<bool> CVarSpanishUsesRAENumberFormat(
	TEXT("Localization.SpanishUsesRAENumberFormat"),
	true,
	TEXT("False: Disabled (CLDR format), True: Enabled (RAE format, default)."),
	ECVF_Default
	);

static TAutoConsoleVariable<bool> CVarSpanishUsesMinTwoGrouping(
	TEXT("Localization.SpanishUsesMinTwoGrouping"),
	true,
	TEXT("False: 1234 will use a group separator, True: 1234 will not use a group separator (default)."),
	ECVF_Default
	);

namespace
{
	const icu::Locale& GetInvariantLocale()
	{
		auto MakeInvariantLocale = []()
		{
			icu::Locale TmpLocale("en-US-POSIX");
			if (TmpLocale.isBogus())
			{
				TmpLocale = icu::Locale();
			}
			return TmpLocale;
		};

		static const icu::Locale InvariantLocale = MakeInvariantLocale();
		return InvariantLocale;
	}

	TSharedRef<const icu::BreakIterator> CreateBreakIterator(const icu::Locale& ICULocale, const EBreakIteratorType Type)
	{
		UErrorCode ICUStatus = U_ZERO_ERROR;
		icu::BreakIterator* (*FactoryFunction)(const icu::Locale&, UErrorCode&) = nullptr;
		switch (Type)
		{
		default:
			ensureAlwaysMsgf(false, TEXT("Unhandled break iterator type"));
			// No break - use Grapheme
		case EBreakIteratorType::Grapheme:
			FactoryFunction = icu::BreakIterator::createCharacterInstance;
			break;
		case EBreakIteratorType::Word:
			FactoryFunction = icu::BreakIterator::createWordInstance;
			break;
		case EBreakIteratorType::Line:
			FactoryFunction = icu::BreakIterator::createLineInstance;
			break;
		case EBreakIteratorType::Sentence:
			FactoryFunction = icu::BreakIterator::createSentenceInstance;
			break;
		case EBreakIteratorType::Title:
			FactoryFunction = icu::BreakIterator::createTitleInstance;
			break;
		}
		TSharedPtr<const icu::BreakIterator> Ptr = MakeShareable(FactoryFunction(ICULocale, ICUStatus));
		if (!ensureAlwaysMsgf(Ptr, TEXT("Creating a break iterator object failed using locale %s. Perhaps this locale has no data."), StringCast<TCHAR>(ICULocale.getName()).Get()))
		{
			ICUStatus = U_ZERO_ERROR;
			Ptr = MakeShareable(FactoryFunction(GetInvariantLocale(), ICUStatus));
			check(Ptr);
		}
		return Ptr.ToSharedRef();
	}

	TSharedRef<const icu::Collator, ESPMode::ThreadSafe> CreateCollator(const icu::Locale& ICULocale)
	{
		UErrorCode ICUStatus = U_ZERO_ERROR;
		TSharedPtr<const icu::Collator, ESPMode::ThreadSafe> Ptr = MakeShareable(icu::Collator::createInstance(ICULocale, ICUStatus));
		if (!ensureAlwaysMsgf(Ptr, TEXT("Creating a collator object failed using locale %s. Perhaps this locale has no data."), StringCast<TCHAR>(ICULocale.getName()).Get()))
		{
			UE_LOGF(LogICUCulture, Warning, "ICU collator instance creation failed locale '%ls', %ls (%d)", StringCast<TCHAR>(ICULocale.getName()).Get(), StringCast<TCHAR>(u_errorName(ICUStatus)).Get(), ICUStatus);
			// Reset the ICUStatus value, this must be non-error or the icu api will immediately fail.
			ICUStatus = U_ZERO_ERROR;
			Ptr = MakeShareable(icu::Collator::createInstance(GetInvariantLocale(), ICUStatus));
			UE_CLOGF(!Ptr, LogICUCulture, Fatal, "ICU collator instance creation failed to create invariant locale %ls (%d)", StringCast<TCHAR>(u_errorName(ICUStatus)).Get(), ICUStatus);
			check(Ptr);
		}
		return Ptr.ToSharedRef();
	}

	TSharedRef<const icu::DateFormat, ESPMode::ThreadSafe> CreateDateFormat(const icu::Locale& ICULocale, const icu::DecimalFormat& ICUDecimalFormat)
	{
		TSharedPtr<icu::DateFormat, ESPMode::ThreadSafe> Ptr = MakeShareable(icu::DateFormat::createDateInstance(icu::DateFormat::kDefault, ICULocale));
		if (!ensureAlwaysMsgf(Ptr, TEXT("Creating a date format object failed using locale %s. Perhaps this locale has no data."), StringCast<TCHAR>(ICULocale.getName()).Get()))
		{
			Ptr = MakeShareable(icu::DateFormat::createDateInstance(icu::DateFormat::kDefault, GetInvariantLocale()));
			check(Ptr);
		}
		Ptr->adoptTimeZone(icu::TimeZone::createDefault());
		Ptr->setNumberFormat(ICUDecimalFormat);
		return Ptr.ToSharedRef();
	}

	TSharedRef<const icu::DateFormat, ESPMode::ThreadSafe> CreateTimeFormat(const icu::Locale& ICULocale, const icu::DecimalFormat& ICUDecimalFormat)
	{
		TSharedPtr<icu::DateFormat, ESPMode::ThreadSafe> Ptr = MakeShareable(icu::DateFormat::createTimeInstance(icu::DateFormat::kDefault, ICULocale));
		if (!ensureAlwaysMsgf(Ptr, TEXT("Creating a time format object failed using locale %s. Perhaps this locale has no data."), StringCast<TCHAR>(ICULocale.getName()).Get()))
		{
			Ptr = MakeShareable(icu::DateFormat::createTimeInstance(icu::DateFormat::kDefault, GetInvariantLocale()));
			check(Ptr);
		}
		Ptr->adoptTimeZone(icu::TimeZone::createDefault());
		Ptr->setNumberFormat(ICUDecimalFormat);
		return Ptr.ToSharedRef();
	}

	TSharedRef<const icu::DateFormat, ESPMode::ThreadSafe> CreateDateTimeFormat(const icu::Locale& ICULocale, const icu::DecimalFormat& ICUDecimalFormat)
	{
		TSharedPtr<icu::DateFormat, ESPMode::ThreadSafe> Ptr = MakeShareable(icu::DateFormat::createDateTimeInstance(icu::DateFormat::kDefault, icu::DateFormat::kDefault, ICULocale));
		if (!ensureAlwaysMsgf(Ptr, TEXT("Creating a date-time format object failed using locale %s. Perhaps this locale has no data."), StringCast<TCHAR>(ICULocale.getName()).Get()))
		{
			Ptr = MakeShareable(icu::DateFormat::createDateTimeInstance(icu::DateFormat::kDefault, icu::DateFormat::kDefault, GetInvariantLocale()));
			check(Ptr);
		}
		Ptr->adoptTimeZone(icu::TimeZone::createDefault());
		Ptr->setNumberFormat(ICUDecimalFormat);
		return Ptr.ToSharedRef();
	}
}

ETextPluralForm ICUPluralFormToUE(const icu::UnicodeString& InICUTag)
{
	static const icu::UnicodeString ZeroStr("zero");
	static const icu::UnicodeString OneStr("one");
	static const icu::UnicodeString TwoStr("two");
	static const icu::UnicodeString FewStr("few");
	static const icu::UnicodeString ManyStr("many");
	static const icu::UnicodeString OtherStr("other");

	if (InICUTag == ZeroStr)
	{
		return ETextPluralForm::Zero;
	}
	if (InICUTag == OneStr)
	{
		return ETextPluralForm::One;
	}
	if (InICUTag == TwoStr)
	{
		return ETextPluralForm::Two;
	}
	if (InICUTag == FewStr)
	{
		return ETextPluralForm::Few;
	}
	if (InICUTag == ManyStr)
	{
		return ETextPluralForm::Many;
	}
	if (InICUTag == OtherStr)
	{
		return ETextPluralForm::Other;
	}

	ensureAlwaysMsgf(false, TEXT("Unknown ICU plural form tag! Returning 'other'."));
	return ETextPluralForm::Other;
}

TArray<ETextPluralForm> ICUPluralRulesToUEValidPluralForms(const icu::PluralRules* InICUPluralRules)
{
	check(InICUPluralRules);

	UErrorCode ICUStatus = U_ZERO_ERROR;
	icu::StringEnumeration* ICUAvailablePluralForms = InICUPluralRules->getKeywords(ICUStatus);

	TArray<ETextPluralForm> UEPluralForms;

	if (ICUAvailablePluralForms)
	{
		while (const icu::UnicodeString* ICUTag = ICUAvailablePluralForms->snext(ICUStatus))
		{
			UEPluralForms.Add(ICUPluralFormToUE(*ICUTag));
		}
		delete ICUAvailablePluralForms;
	}

	UEPluralForms.Sort();
	return UEPluralForms;
}

FICUCultureImplementation::FICUCultureImplementation(const FString& LocaleName)
	: ICULocale( TCHAR_TO_ANSI( *LocaleName ) )
{
	// If anyone ever tries to remove this in future you need to remove the `AUTORTFM_OPEN` from
	// `GetDecimalNumberFormattingRules`, `GetPercentFormattingRules`, and `GetCurrencyFormattingRules`,
	// and actually make all that code transactionally safe (its gonna be tricky!).
	AutoRTFM::UnreachableIfClosed("Cannot construct `FICUCultureImplementation` in the closed!");

	if (ICULocale.isBogus())
	{
		ICULocale = GetInvariantLocale();
	}
	{
		UErrorCode ICUStatus = U_ZERO_ERROR;
		ICUCardinalPluralRules = icu::PluralRules::forLocale(ICULocale, UPLURAL_TYPE_CARDINAL, ICUStatus);
		if (!ensureAlwaysMsgf(U_SUCCESS(ICUStatus) && ICUCardinalPluralRules, TEXT("Creating a cardinal plural rules object failed using locale %s. Perhaps this locale has no data."), *LocaleName))
		{
			ICUStatus = U_ZERO_ERROR;
			ICUCardinalPluralRules = icu::PluralRules::forLocale(GetInvariantLocale(), UPLURAL_TYPE_CARDINAL, ICUStatus);
			check(ICUCardinalPluralRules);
		}
		UEAvailableCardinalPluralForms = ICUPluralRulesToUEValidPluralForms(ICUCardinalPluralRules);
	}
	{
		UErrorCode ICUStatus = U_ZERO_ERROR;
		ICUOrdinalPluralRules = icu::PluralRules::forLocale(ICULocale, UPLURAL_TYPE_ORDINAL, ICUStatus);
		if (!ensureAlwaysMsgf(U_SUCCESS(ICUStatus) && ICUOrdinalPluralRules, TEXT("Creating an ordinal plural rules object failed using locale %s. Perhaps this locale has no data."), *LocaleName))
		{
			ICUStatus = U_ZERO_ERROR;
			ICUOrdinalPluralRules = icu::PluralRules::forLocale(GetInvariantLocale(), UPLURAL_TYPE_ORDINAL, ICUStatus);
			check(ICUOrdinalPluralRules);
		}
		UEAvailableOrdinalPluralForms = ICUPluralRulesToUEValidPluralForms(ICUOrdinalPluralRules);
	}
}

FString FICUCultureImplementation::GetDisplayName() const
{
	icu::UnicodeString ICUResult;
	ICULocale.getDisplayName(ICUResult);
	return ICUUtilities::ConvertString(ICUResult);
}

FString FICUCultureImplementation::GetEnglishName() const
{
	icu::UnicodeString ICUResult;
	ICULocale.getDisplayName(icu::Locale("en"), ICUResult);
	return ICUUtilities::ConvertString(ICUResult);
}

int FICUCultureImplementation::GetKeyboardLayoutId() const
{
	return 0;
}

int FICUCultureImplementation::GetLCID() const
{
	return ICULocale.getLCID();
}

FString FICUCultureImplementation::GetCanonicalName(const FString& Name, FInternationalization& I18N)
{
	return ICUUtilities::GetCanonicalCultureName(Name, TEXT("en-US-POSIX"), I18N);
}

FString FICUCultureImplementation::GetName() const
{
	// UE uses "en-US" style while ICU uses "en_US"
	FString Result = ICULocale.getName();
	Result.ReplaceCharInline(TEXT('_'), TEXT('-'), ESearchCase::CaseSensitive);
	return Result;
}

FString FICUCultureImplementation::GetNativeName() const
{
	icu::UnicodeString ICUResult;
	ICULocale.getDisplayName(ICULocale, ICUResult);
	return ICUUtilities::ConvertString(ICUResult);
}

FString FICUCultureImplementation::GetUnrealLegacyThreeLetterISOLanguageName() const
{
	FString Result( ICULocale.getISO3Language() );

	// Legacy Overrides (INT, JPN, KOR), also for new web localization (CHN)
	// and now for any other languages (FRA, DEU...) for correct redirection of documentation web links
	if (Result == TEXT("eng"))
	{
		Result = TEXT("INT");
	}
	else
	{
		Result = Result.ToUpper();
	}

	return Result;
}

FString FICUCultureImplementation::GetThreeLetterISOLanguageName() const
{
	return ICULocale.getISO3Language();
}

FString FICUCultureImplementation::GetTwoLetterISOLanguageName() const
{
	return ICULocale.getLanguage();
}

FString FICUCultureImplementation::GetNativeLanguage() const
{
	icu::UnicodeString ICUNativeLanguage;
	ICULocale.getDisplayLanguage(ICULocale, ICUNativeLanguage);
	FString NativeLanguage;
	ICUUtilities::ConvertString(ICUNativeLanguage, NativeLanguage);

	icu::UnicodeString ICUNativeScript;
	ICULocale.getDisplayScript(ICULocale, ICUNativeScript);
	FString NativeScript;
	ICUUtilities::ConvertString(ICUNativeScript, NativeScript);

	if ( !NativeScript.IsEmpty() )
	{
		return NativeLanguage + TEXT(" (") + NativeScript + TEXT(")");
	}
	return NativeLanguage;
}

FString FICUCultureImplementation::GetRegion() const
{
	return ICULocale.getCountry();
}

FString FICUCultureImplementation::GetNativeRegion() const
{
	icu::UnicodeString ICUNativeCountry;
	ICULocale.getDisplayCountry(ICULocale, ICUNativeCountry);
	FString NativeCountry;
	ICUUtilities::ConvertString(ICUNativeCountry, NativeCountry);

	icu::UnicodeString ICUNativeVariant;
	ICULocale.getDisplayVariant(ICULocale, ICUNativeVariant);
	FString NativeVariant;
	ICUUtilities::ConvertString(ICUNativeVariant, NativeVariant);

	if ( !NativeVariant.IsEmpty() )
	{
		return NativeCountry + TEXT(", ") + NativeVariant;
	}
	return NativeCountry;
}

FString FICUCultureImplementation::GetScript() const
{
	return ICULocale.getScript();
}

FString FICUCultureImplementation::GetVariant() const
{
	return ICULocale.getVariant();
}

bool FICUCultureImplementation::IsRightToLeft() const
{
#if WITH_ICU_V64 || WITH_ICU_V78
	return ICULocale.isRightToLeft() != 0;
#else
	return false;
#endif
}

TSharedRef<const icu::BreakIterator> FICUCultureImplementation::GetBreakIterator(const EBreakIteratorType Type)
{
	TSharedPtr<const icu::BreakIterator> Result;

	switch (Type)
	{
	case EBreakIteratorType::Grapheme:
		{
			Result = ICUGraphemeBreakIterator.IsValid() ? ICUGraphemeBreakIterator : ( ICUGraphemeBreakIterator = CreateBreakIterator(ICULocale, Type) );
		}
		break;
	case EBreakIteratorType::Word:
		{
			Result = ICUWordBreakIterator.IsValid() ? ICUWordBreakIterator : ( ICUWordBreakIterator = CreateBreakIterator(ICULocale, Type) );
		}
		break;
	case EBreakIteratorType::Line:
		{
			Result = ICULineBreakIterator.IsValid() ? ICULineBreakIterator : ( ICULineBreakIterator = CreateBreakIterator(ICULocale, Type) );
		}
		break;
	case EBreakIteratorType::Sentence:
		{
			Result = ICUSentenceBreakIterator.IsValid() ? ICUSentenceBreakIterator : ( ICUSentenceBreakIterator = CreateBreakIterator(ICULocale, Type) );
		}
		break;
	case EBreakIteratorType::Title:
		{
			Result = ICUTitleBreakIterator.IsValid() ? ICUTitleBreakIterator : ( ICUTitleBreakIterator = CreateBreakIterator(ICULocale, Type) );
		}
		break;
	}

	return Result.ToSharedRef();
}

TSharedRef<const icu::Collator, ESPMode::ThreadSafe> FICUCultureImplementation::GetCollator(const ETextComparisonLevel::Type ComparisonLevel)
{
	if (!ICUCollator.IsValid())
	{
		ICUCollator = CreateCollator( ICULocale );
	}

	const bool bIsDefault = (ComparisonLevel == ETextComparisonLevel::Default);
	const TSharedRef<const icu::Collator, ESPMode::ThreadSafe> DefaultCollator( ICUCollator.ToSharedRef() );
	if(bIsDefault)
	{
		return DefaultCollator;
	}
	else
	{
		const TSharedRef<icu::Collator, ESPMode::ThreadSafe> Collator( DefaultCollator->clone() );

		UErrorCode ICUStatus = U_ZERO_ERROR;
		Collator->setAttribute(UColAttribute::UCOL_STRENGTH, UEToICU(ComparisonLevel), ICUStatus);
		
		return Collator;
	}
}

TSharedRef<const icu::DecimalFormat, ESPMode::ThreadSafe> FICUCultureImplementation::GetDateTimeDecimalFormat()
{
	if (!ICUDateTimeDecimalFormat)
	{
		// Create a culture decimal formatter
		TSharedPtr<icu::DecimalFormat> DecimalFormatterForCulture;
		{
			UErrorCode ICUStatus = U_ZERO_ERROR;
			DecimalFormatterForCulture = MakeShareable(static_cast<icu::DecimalFormat*>(icu::NumberFormat::createInstance(ICULocale, ICUStatus)));
			if (!ensureAlwaysMsgf(DecimalFormatterForCulture, TEXT("Creating a decimal format object failed using locale %s. Perhaps this locale has no data."), StringCast<TCHAR>(ICULocale.getName()).Get()))
			{
				ICUStatus = U_ZERO_ERROR;
				DecimalFormatterForCulture = MakeShareable(static_cast<icu::DecimalFormat*>(icu::NumberFormat::createInstance(GetInvariantLocale(), ICUStatus)));
				check(DecimalFormatterForCulture);
			}
		}

		// Sync the ICU decimal symbols with the ones we want to use for UE
		{
			const FDecimalNumberFormattingRules& NewUEDecimalNumberFormattingRules = GetDecimalNumberFormattingRules();

			icu::DecimalFormatSymbols ICUDecimalSymbols = *DecimalFormatterForCulture->getDecimalFormatSymbols();

			ICUDecimalSymbols.setSymbol(icu::DecimalFormatSymbols::kZeroDigitSymbol, ICUUtilities::ConvertString(FStringView(&NewUEDecimalNumberFormattingRules.DigitCharacters[0], 1)), false);
			ICUDecimalSymbols.setSymbol(icu::DecimalFormatSymbols::kOneDigitSymbol, ICUUtilities::ConvertString(FStringView(&NewUEDecimalNumberFormattingRules.DigitCharacters[1], 1)), false);
			ICUDecimalSymbols.setSymbol(icu::DecimalFormatSymbols::kTwoDigitSymbol, ICUUtilities::ConvertString(FStringView(&NewUEDecimalNumberFormattingRules.DigitCharacters[2], 1)), false);
			ICUDecimalSymbols.setSymbol(icu::DecimalFormatSymbols::kThreeDigitSymbol, ICUUtilities::ConvertString(FStringView(&NewUEDecimalNumberFormattingRules.DigitCharacters[3], 1)), false);
			ICUDecimalSymbols.setSymbol(icu::DecimalFormatSymbols::kFourDigitSymbol, ICUUtilities::ConvertString(FStringView(&NewUEDecimalNumberFormattingRules.DigitCharacters[4], 1)), false);
			ICUDecimalSymbols.setSymbol(icu::DecimalFormatSymbols::kFiveDigitSymbol, ICUUtilities::ConvertString(FStringView(&NewUEDecimalNumberFormattingRules.DigitCharacters[5], 1)), false);
			ICUDecimalSymbols.setSymbol(icu::DecimalFormatSymbols::kSixDigitSymbol, ICUUtilities::ConvertString(FStringView(&NewUEDecimalNumberFormattingRules.DigitCharacters[6], 1)), false);
			ICUDecimalSymbols.setSymbol(icu::DecimalFormatSymbols::kSevenDigitSymbol, ICUUtilities::ConvertString(FStringView(&NewUEDecimalNumberFormattingRules.DigitCharacters[7], 1)), false);
			ICUDecimalSymbols.setSymbol(icu::DecimalFormatSymbols::kEightDigitSymbol, ICUUtilities::ConvertString(FStringView(&NewUEDecimalNumberFormattingRules.DigitCharacters[8], 1)), false);
			ICUDecimalSymbols.setSymbol(icu::DecimalFormatSymbols::kNineDigitSymbol, ICUUtilities::ConvertString(FStringView(&NewUEDecimalNumberFormattingRules.DigitCharacters[9], 1)), false);

			ICUDecimalSymbols.setSymbol(icu::DecimalFormatSymbols::kGroupingSeparatorSymbol, ICUUtilities::ConvertString(FStringView(&NewUEDecimalNumberFormattingRules.GroupingSeparatorCharacter, 1)), false);
			ICUDecimalSymbols.setSymbol(icu::DecimalFormatSymbols::kDecimalSeparatorSymbol, ICUUtilities::ConvertString(FStringView(&NewUEDecimalNumberFormattingRules.DecimalSeparatorCharacter, 1)), false);

			DecimalFormatterForCulture->setDecimalFormatSymbols(ICUDecimalSymbols);

			DecimalFormatterForCulture->setGroupingUsed(false);
		}

		ICUDateTimeDecimalFormat = DecimalFormatterForCulture;
	}

	return ICUDateTimeDecimalFormat.ToSharedRef();
}

TSharedRef<const icu::DateFormat, ESPMode::ThreadSafe> FICUCultureImplementation::GetDateFormatter(const EDateTimeStyle::Type DateStyle, const FString& TimeZone)
{
	if (!ICUDateFormat.IsValid())
	{
		ICUDateFormat = CreateDateFormat( ICULocale, *GetDateTimeDecimalFormat() );
	}

	const FString SanitizedTimezoneCode = ICUUtilities::SanitizeTimezoneCode(TimeZone);

	icu::UnicodeString InputTimeZoneID;
	ICUUtilities::ConvertString(SanitizedTimezoneCode, InputTimeZoneID, false);

	const TSharedRef<const icu::DateFormat, ESPMode::ThreadSafe> DefaultFormatter( ICUDateFormat.ToSharedRef() );

	bool bIsDefaultTimeZone = SanitizedTimezoneCode.IsEmpty();
	if( !bIsDefaultTimeZone )
	{
		icu::UnicodeString CanonicalInputTimeZoneID;
		{
			UErrorCode ICUStatus = U_ZERO_ERROR;
			icu::TimeZone::getCanonicalID(InputTimeZoneID, CanonicalInputTimeZoneID, ICUStatus);
		}

		icu::UnicodeString DefaultTimeZoneID;
		DefaultFormatter->getTimeZone().getID(DefaultTimeZoneID);

		icu::UnicodeString CanonicalDefaultTimeZoneID;
		{
			UErrorCode ICUStatus = U_ZERO_ERROR;
			icu::TimeZone::getCanonicalID(DefaultTimeZoneID, CanonicalDefaultTimeZoneID, ICUStatus);
		}

		bIsDefaultTimeZone = (CanonicalInputTimeZoneID == CanonicalDefaultTimeZoneID ? true : false);
	}

	const bool bIsDefault = 
		DateStyle == EDateTimeStyle::Default &&
		bIsDefaultTimeZone;

	if(bIsDefault)
	{
		return DefaultFormatter;
	}
	else
	{
		TSharedPtr<icu::DateFormat, ESPMode::ThreadSafe> Formatter = MakeShareable(icu::DateFormat::createDateInstance(UEToICU(DateStyle), ICULocale));
		if (!ensureAlwaysMsgf(Formatter, TEXT("Creating a date format object failed using locale %s. Perhaps this locale has no data."), StringCast<TCHAR>(ICULocale.getName()).Get()))
		{
			Formatter = MakeShareable(icu::DateFormat::createDateInstance(UEToICU(DateStyle), GetInvariantLocale()));
			check(Formatter);
		}
		Formatter->adoptTimeZone(bIsDefaultTimeZone ? icu::TimeZone::createDefault() : icu::TimeZone::createTimeZone(InputTimeZoneID));
		return Formatter.ToSharedRef();
	}
}

TSharedRef<const icu::DateFormat, ESPMode::ThreadSafe> FICUCultureImplementation::GetTimeFormatter(const EDateTimeStyle::Type TimeStyle, const FString& TimeZone)
{
	if (!ICUTimeFormat.IsValid())
	{
		ICUTimeFormat = CreateTimeFormat( ICULocale, *GetDateTimeDecimalFormat() );
	}

	const FString SanitizedTimezoneCode = ICUUtilities::SanitizeTimezoneCode(TimeZone);

	icu::UnicodeString InputTimeZoneID;
	ICUUtilities::ConvertString(SanitizedTimezoneCode, InputTimeZoneID, false);

	const TSharedRef<const icu::DateFormat, ESPMode::ThreadSafe> DefaultFormatter( ICUTimeFormat.ToSharedRef() );

	bool bIsDefaultTimeZone = SanitizedTimezoneCode.IsEmpty();
	if( !bIsDefaultTimeZone )
	{
		icu::UnicodeString CanonicalInputTimeZoneID;
		{
			UErrorCode ICUStatus = U_ZERO_ERROR;
			icu::TimeZone::getCanonicalID(InputTimeZoneID, CanonicalInputTimeZoneID, ICUStatus);
		}

		icu::UnicodeString DefaultTimeZoneID;
		DefaultFormatter->getTimeZone().getID(DefaultTimeZoneID);

		icu::UnicodeString CanonicalDefaultTimeZoneID;
		{
			UErrorCode ICUStatus = U_ZERO_ERROR;
			icu::TimeZone::getCanonicalID(DefaultTimeZoneID, CanonicalDefaultTimeZoneID, ICUStatus);
		}

		bIsDefaultTimeZone = (CanonicalInputTimeZoneID == CanonicalDefaultTimeZoneID ? true : false);
	}

	const bool bIsDefault = 
		TimeStyle == EDateTimeStyle::Default &&
		bIsDefaultTimeZone;

	if(bIsDefault)
	{
		return DefaultFormatter;
	}
	else
	{
		TSharedPtr<icu::DateFormat, ESPMode::ThreadSafe> Formatter = MakeShareable(icu::DateFormat::createTimeInstance(UEToICU(TimeStyle), ICULocale));
		if (!ensureAlwaysMsgf(Formatter, TEXT("Creating a time format object failed using locale %s. Perhaps this locale has no data."), StringCast<TCHAR>(ICULocale.getName()).Get()))
		{
			Formatter = MakeShareable(icu::DateFormat::createTimeInstance(UEToICU(TimeStyle), GetInvariantLocale()));
			check(Formatter);
		}
		Formatter->adoptTimeZone(bIsDefaultTimeZone ? icu::TimeZone::createDefault() : icu::TimeZone::createTimeZone(InputTimeZoneID));
		Formatter->setNumberFormat(*GetDateTimeDecimalFormat());
		return Formatter.ToSharedRef();
	}
}

TSharedRef<const icu::DateFormat, ESPMode::ThreadSafe> FICUCultureImplementation::GetDateTimeFormatter(const EDateTimeStyle::Type DateStyle, const EDateTimeStyle::Type TimeStyle, const FString& TimeZone)
{
	if (!ICUDateTimeFormat.IsValid())
	{
		ICUDateTimeFormat = CreateDateTimeFormat( ICULocale, *GetDateTimeDecimalFormat() );
	}

	const FString SanitizedTimezoneCode = ICUUtilities::SanitizeTimezoneCode(TimeZone);

	icu::UnicodeString InputTimeZoneID;
	ICUUtilities::ConvertString(SanitizedTimezoneCode, InputTimeZoneID, false);

	const TSharedRef<const icu::DateFormat, ESPMode::ThreadSafe> DefaultFormatter( ICUDateTimeFormat.ToSharedRef() );

	bool bIsDefaultTimeZone = SanitizedTimezoneCode.IsEmpty();
	if( !bIsDefaultTimeZone )
	{
		icu::UnicodeString CanonicalInputTimeZoneID;
		{
			UErrorCode ICUStatus = U_ZERO_ERROR;
			icu::TimeZone::getCanonicalID(InputTimeZoneID, CanonicalInputTimeZoneID, ICUStatus);
		}

		icu::UnicodeString DefaultTimeZoneID;
		DefaultFormatter->getTimeZone().getID(DefaultTimeZoneID);

		icu::UnicodeString CanonicalDefaultTimeZoneID;
		{
			UErrorCode ICUStatus = U_ZERO_ERROR;
			icu::TimeZone::getCanonicalID(DefaultTimeZoneID, CanonicalDefaultTimeZoneID, ICUStatus);
		}

		bIsDefaultTimeZone = (CanonicalInputTimeZoneID == CanonicalDefaultTimeZoneID ? true : false);
	}

	const bool bIsDefault = 
		DateStyle == EDateTimeStyle::Default &&
		TimeStyle == EDateTimeStyle::Default &&
		bIsDefaultTimeZone;

	if(bIsDefault)
	{
		return DefaultFormatter;
	}
	else
	{
		TSharedPtr<icu::DateFormat, ESPMode::ThreadSafe> Formatter = MakeShareable(icu::DateFormat::createDateTimeInstance(UEToICU(DateStyle), UEToICU(TimeStyle), ICULocale));
		if (!ensureAlwaysMsgf(Formatter, TEXT("Creating a date-time format object failed using locale %s. Perhaps this locale has no data."), StringCast<TCHAR>(ICULocale.getName()).Get()))
		{
			Formatter = MakeShareable(icu::DateFormat::createDateTimeInstance(UEToICU(DateStyle), UEToICU(TimeStyle), GetInvariantLocale()));
			check(Formatter);
		}
		Formatter->adoptTimeZone(bIsDefaultTimeZone ? icu::TimeZone::createDefault() : icu::TimeZone::createTimeZone(InputTimeZoneID));
		Formatter->setNumberFormat(*GetDateTimeDecimalFormat());
		return Formatter.ToSharedRef();
	}
}

TSharedRef<const icu::DateFormat, ESPMode::ThreadSafe> FICUCultureImplementation::GetDateTimeFormatter(const FString& CustomPattern, const FString& TimeZone)
{
#if WITH_ICU_V64 || WITH_ICU_V78
	// createInstanceForSkeleton was added in ICU 55, so we need to make sure we're using a newer version of ICU (prior to ICU 64 we used ICU 53)
	{
		auto DateTimePatternToICUSkeleton = [](const TCHAR* Format) -> FString
		{
			TStringBuilder<32> Result;

			while (*Format != TCHAR('\0'))
			{
				if ((*Format == TCHAR('%')) && (*(++Format) != TCHAR('\0')))
				{
					switch (*Format)
					{
					case TCHAR('a'): Result.Append(TEXT("EEE")); break;
					case TCHAR('A'): Result.Append(TEXT("EEEE")); break;
					case TCHAR('w'): Result.Append(TEXT("e")); break;
					case TCHAR('y'): Result.Append(TEXT("yy")); break;
					case TCHAR('Y'): Result.Append(TEXT("yyyy")); break;
					case TCHAR('b'): Result.Append(TEXT("MMM")); break;
					case TCHAR('B'): Result.Append(TEXT("MMMM")); break;
					case TCHAR('m'): Result.Append(TEXT("MM")); break;
					case TCHAR('d'): Result.Append(TEXT("dd")); break;
					case TCHAR('e'): Result.Append(TEXT("d")); break;
					case TCHAR('l'): Result.Append(TEXT("h")); break;
					case TCHAR('I'): Result.Append(TEXT("hh")); break;
					case TCHAR('H'): Result.Append(TEXT("HH")); break;
					case TCHAR('M'): Result.Append(TEXT("mm")); break;
					case TCHAR('S'): Result.Append(TEXT("ss")); break;
					case TCHAR('p'): Result.Append(TEXT("a")); break;
					case TCHAR('P'): Result.Append(TEXT("a")); break;
					case TCHAR('j'): Result.Append(TEXT("D")); break;
					default:		 Result.AppendChar(*Format);
					}
				}
				else
				{
					Result.AppendChar(*Format);
				}

				// move to the next one
				Format++;
			}

			return Result.ToString();
		};

		const FString DateTimeSkeleton = DateTimePatternToICUSkeleton(*CustomPattern);

		TSharedPtr<icu::DateFormat, ESPMode::ThreadSafe> Formatter;
		{
			UErrorCode ICUStatus = U_ZERO_ERROR;
			Formatter = MakeShareable(icu::DateFormat::createInstanceForSkeleton(ICUUtilities::ConvertString(DateTimeSkeleton), ICULocale, ICUStatus));
			if (!Formatter)
			{
				ICUStatus = U_ZERO_ERROR;
				Formatter = MakeShareable(icu::DateFormat::createInstanceForSkeleton(ICUUtilities::ConvertString(DateTimeSkeleton), GetInvariantLocale(), ICUStatus));
			}
		}

		if (Formatter)
		{
			const FString SanitizedTimezoneCode = ICUUtilities::SanitizeTimezoneCode(TimeZone);
			Formatter->adoptTimeZone(SanitizedTimezoneCode.IsEmpty() ? icu::TimeZone::createDefault() : icu::TimeZone::createTimeZone(ICUUtilities::ConvertString(SanitizedTimezoneCode, false)));

			Formatter->setNumberFormat(*GetDateTimeDecimalFormat());

			return Formatter.ToSharedRef();
		}
	}
#endif	// WITH_ICU_V64 || WITH_ICU_V78

	return GetDateTimeFormatter(EDateTimeStyle::Default, EDateTimeStyle::Default, TimeZone);
}

namespace
{

FDecimalNumberFormattingRules ExtractNumberFormattingRulesFromICUDecimalFormatter(icu::Locale& InICULocale, icu::DecimalFormat& InICUDecimalFormat)
{
	FDecimalNumberFormattingRules NewUEDecimalNumberFormattingRules;

	// Extract the default formatting options before we mess around with the formatter object settings
	NewUEDecimalNumberFormattingRules.CultureDefaultFormattingOptions
		.SetUseGrouping(InICUDecimalFormat.isGroupingUsed() != 0)
		.SetRoundingMode(ICUToUE(InICUDecimalFormat.getRoundingMode()))
		.SetMinimumIntegralDigits(InICUDecimalFormat.getMinimumIntegerDigits())
		.SetMaximumIntegralDigits(InICUDecimalFormat.getMaximumIntegerDigits())
		.SetMinimumFractionalDigits(InICUDecimalFormat.getMinimumFractionDigits())
		.SetMaximumFractionalDigits(InICUDecimalFormat.getMaximumFractionDigits());

	// We force grouping to be on, even if a culture doesn't use it by default, so that we can extract meaningful grouping information
	// This allows us to use the correct groupings if we should ever force grouping for a number, rather than use the culture default
	InICUDecimalFormat.setGroupingUsed(true);

	auto ExtractFormattingSymbolAsCharacter = [&](icu::DecimalFormatSymbols::ENumberFormatSymbol InSymbolToExtract, const TCHAR InFallbackChar) -> TCHAR
	{
		const icu::UnicodeString& ICUSymbolString = InICUDecimalFormat.getDecimalFormatSymbols()->getConstSymbol(InSymbolToExtract);
		check(ICUSymbolString.length() <= 2);

		// Some cultures use characters outside of the BMP which present as a surrogate pair on platforms using UTF-16 TCHAR
		// We need to update this code to use FString or UTF32CHAR for these symbols (see UE-83143), but for now we just use the fallback if we find a surrogate pair
		return ICUSymbolString.length() == 1
			? static_cast<TCHAR>(ICUSymbolString.charAt(0))
			: InFallbackChar;
	};

	icu::UnicodeString ScratchICUString;

	// Extract the rules from the decimal formatter
	NewUEDecimalNumberFormattingRules.NaNString						= ICUUtilities::ConvertString(InICUDecimalFormat.getDecimalFormatSymbols()->getConstSymbol(icu::DecimalFormatSymbols::kNaNSymbol));
	NewUEDecimalNumberFormattingRules.NegativePrefixString			= ICUUtilities::ConvertString(InICUDecimalFormat.getNegativePrefix(ScratchICUString));
	NewUEDecimalNumberFormattingRules.NegativeSuffixString			= ICUUtilities::ConvertString(InICUDecimalFormat.getNegativeSuffix(ScratchICUString));
	NewUEDecimalNumberFormattingRules.PositivePrefixString			= ICUUtilities::ConvertString(InICUDecimalFormat.getPositivePrefix(ScratchICUString));
	NewUEDecimalNumberFormattingRules.PositiveSuffixString			= ICUUtilities::ConvertString(InICUDecimalFormat.getPositiveSuffix(ScratchICUString));
	NewUEDecimalNumberFormattingRules.PlusString					= ICUUtilities::ConvertString(InICUDecimalFormat.getDecimalFormatSymbols()->getConstSymbol(icu::DecimalFormatSymbols::kPlusSignSymbol));
	NewUEDecimalNumberFormattingRules.MinusString					= ICUUtilities::ConvertString(InICUDecimalFormat.getDecimalFormatSymbols()->getConstSymbol(icu::DecimalFormatSymbols::kMinusSignSymbol));
	NewUEDecimalNumberFormattingRules.GroupingSeparatorCharacter	= ExtractFormattingSymbolAsCharacter(icu::DecimalFormatSymbols::kGroupingSeparatorSymbol, TEXT(','));
	NewUEDecimalNumberFormattingRules.DecimalSeparatorCharacter		= ExtractFormattingSymbolAsCharacter(icu::DecimalFormatSymbols::kDecimalSeparatorSymbol,  TEXT('.'));
	NewUEDecimalNumberFormattingRules.PrimaryGroupingSize			= static_cast<uint8>(InICUDecimalFormat.getGroupingSize());
	NewUEDecimalNumberFormattingRules.SecondaryGroupingSize			= (InICUDecimalFormat.getSecondaryGroupingSize() < 1) 
																		? NewUEDecimalNumberFormattingRules.PrimaryGroupingSize 
																		: static_cast<uint8>(InICUDecimalFormat.getSecondaryGroupingSize());
#if WITH_ICU_V64 || WITH_ICU_V78
	NewUEDecimalNumberFormattingRules.MinimumGroupingDigits			= static_cast<uint8>(FMath::Max(InICUDecimalFormat.getMinimumGroupingDigits(), 1));
#endif

	if (CVarUseLocaleSpecificDigitCharacters.AsVariable()->GetBool())
	{
		NewUEDecimalNumberFormattingRules.DigitCharacters[0]		= ExtractFormattingSymbolAsCharacter(icu::DecimalFormatSymbols::kZeroDigitSymbol,	TEXT('0'));
		NewUEDecimalNumberFormattingRules.DigitCharacters[1]		= ExtractFormattingSymbolAsCharacter(icu::DecimalFormatSymbols::kOneDigitSymbol,	TEXT('1'));
		NewUEDecimalNumberFormattingRules.DigitCharacters[2]		= ExtractFormattingSymbolAsCharacter(icu::DecimalFormatSymbols::kTwoDigitSymbol,	TEXT('2'));
		NewUEDecimalNumberFormattingRules.DigitCharacters[3]		= ExtractFormattingSymbolAsCharacter(icu::DecimalFormatSymbols::kThreeDigitSymbol,	TEXT('3'));
		NewUEDecimalNumberFormattingRules.DigitCharacters[4]		= ExtractFormattingSymbolAsCharacter(icu::DecimalFormatSymbols::kFourDigitSymbol,	TEXT('4'));
		NewUEDecimalNumberFormattingRules.DigitCharacters[5]		= ExtractFormattingSymbolAsCharacter(icu::DecimalFormatSymbols::kFiveDigitSymbol,	TEXT('5'));
		NewUEDecimalNumberFormattingRules.DigitCharacters[6]		= ExtractFormattingSymbolAsCharacter(icu::DecimalFormatSymbols::kSixDigitSymbol,	TEXT('6'));
		NewUEDecimalNumberFormattingRules.DigitCharacters[7]		= ExtractFormattingSymbolAsCharacter(icu::DecimalFormatSymbols::kSevenDigitSymbol,	TEXT('7'));
		NewUEDecimalNumberFormattingRules.DigitCharacters[8]		= ExtractFormattingSymbolAsCharacter(icu::DecimalFormatSymbols::kEightDigitSymbol,	TEXT('8'));
		NewUEDecimalNumberFormattingRules.DigitCharacters[9]		= ExtractFormattingSymbolAsCharacter(icu::DecimalFormatSymbols::kNineDigitSymbol,	TEXT('9'));
	}
	else
	{
		auto ReplaceLocaleSeparatorWithSuitableEquivalent = [](TCHAR& InOutSeparatorCharacter)
		{
			switch (InOutSeparatorCharacter)
			{
			case TEXT('\u066B'): // Arabic decimal separator
				InOutSeparatorCharacter = TEXT('.');
				break;
			case TEXT('\u066C'): // Arabic group separator
				InOutSeparatorCharacter = TEXT(',');
				break;
			default:
				break;
			}
		};

		ReplaceLocaleSeparatorWithSuitableEquivalent(NewUEDecimalNumberFormattingRules.GroupingSeparatorCharacter);
		ReplaceLocaleSeparatorWithSuitableEquivalent(NewUEDecimalNumberFormattingRules.DecimalSeparatorCharacter);
	}

	if (FCStringAnsi::Strcmp(InICULocale.getLanguage(), "es") == 0)
	{
		// The CLDR uses a dot as the group separator for Spanish, however the RAE favor using a space: https://www.rae.es/dpd/n%C3%BAmeros
		if (CVarSpanishUsesRAENumberFormat.AsVariable()->GetBool())
		{
			NewUEDecimalNumberFormattingRules.GroupingSeparatorCharacter = TEXT('\u00A0'); // No-Break Space
			NewUEDecimalNumberFormattingRules.DecimalSeparatorCharacter = TEXT(',');
		}

		// Should we use "min two" grouping for Spanish (eg, "1234" formats as "1234" rather than "1 234", but "12345" still formats as "12 345")
		if (CVarSpanishUsesMinTwoGrouping.AsVariable()->GetBool())
		{
			NewUEDecimalNumberFormattingRules.MinimumGroupingDigits = 2;
		}
	}

	return NewUEDecimalNumberFormattingRules;
}

} // anonymous namespace

const FDecimalNumberFormattingRules& FICUCultureImplementation::GetDecimalNumberFormattingRules()
{
	if (UEDecimalNumberFormattingRules.IsValid())
	{
		return *UEDecimalNumberFormattingRules;
	}

	// Create a culture decimal formatter
	TSharedPtr<icu::DecimalFormat> DecimalFormatterForCulture;
	{
		UErrorCode ICUStatus = U_ZERO_ERROR;
		DecimalFormatterForCulture = MakeShareable(static_cast<icu::DecimalFormat*>(icu::NumberFormat::createInstance(ICULocale, ICUStatus)));
		if (!ensureAlwaysMsgf(DecimalFormatterForCulture, TEXT("Creating a decimal format object failed using locale %s. Perhaps this locale has no data."), StringCast<TCHAR>(ICULocale.getName()).Get()))
		{
			ICUStatus = U_ZERO_ERROR;
			DecimalFormatterForCulture = MakeShareable(static_cast<icu::DecimalFormat*>(icu::NumberFormat::createInstance(GetInvariantLocale(), ICUStatus)));
			check(DecimalFormatterForCulture);
		}
	}

	const FDecimalNumberFormattingRules NewUEDecimalNumberFormattingRules = ExtractNumberFormattingRulesFromICUDecimalFormatter(ICULocale, *DecimalFormatterForCulture);

	// Check the pointer again in case another thread beat us to it
	{
		FScopeLock PtrLock(&UEDecimalNumberFormattingRulesCS);

		if (!UEDecimalNumberFormattingRules.IsValid())
		{
			UEDecimalNumberFormattingRules = MakeShared<FDecimalNumberFormattingRules, ESPMode::ThreadSafe>(NewUEDecimalNumberFormattingRules);
		}
	}

	return *UEDecimalNumberFormattingRules;
}

const FDecimalNumberFormattingRules& FICUCultureImplementation::GetPercentFormattingRules()
{
	if (UEPercentFormattingRules.IsValid())
	{
		return *UEPercentFormattingRules;
	}

	// Create a culture percent formatter (doesn't call CreatePercentFormat as we need a mutable instance)
	TSharedPtr<icu::DecimalFormat> PercentFormatterForCulture;
	{
		UErrorCode ICUStatus = U_ZERO_ERROR;
		PercentFormatterForCulture = MakeShareable(static_cast<icu::DecimalFormat*>(icu::NumberFormat::createPercentInstance(ICULocale, ICUStatus)));
		if (!ensureAlwaysMsgf(PercentFormatterForCulture, TEXT("Creating a percent format object failed using locale %s. Perhaps this locale has no data."), StringCast<TCHAR>(ICULocale.getName()).Get()))
		{
			ICUStatus = U_ZERO_ERROR;
			PercentFormatterForCulture = MakeShareable(static_cast<icu::DecimalFormat*>(icu::NumberFormat::createPercentInstance(GetInvariantLocale(), ICUStatus)));
			check(PercentFormatterForCulture);
		}
	}

	const FDecimalNumberFormattingRules NewUEPercentFormattingRules = ExtractNumberFormattingRulesFromICUDecimalFormatter(ICULocale, *PercentFormatterForCulture);

	// Check the pointer again in case another thread beat us to it
	{
		FScopeLock PtrLock(&UEPercentFormattingRulesCS);

		if (!UEPercentFormattingRules.IsValid())
		{
			UEPercentFormattingRules = MakeShared<FDecimalNumberFormattingRules, ESPMode::ThreadSafe>(NewUEPercentFormattingRules);
		}
	}

	return *UEPercentFormattingRules;
}

const FDecimalNumberFormattingRules& FICUCultureImplementation::GetCurrencyFormattingRules(const FString& InCurrencyCode)
{
	const FString SanitizedCurrencyCode = ICUUtilities::SanitizeCurrencyCode(InCurrencyCode);
	const bool bUseDefaultFormattingRules = SanitizedCurrencyCode.IsEmpty();

	if (bUseDefaultFormattingRules)
	{
		if (UECurrencyFormattingRules.IsValid())
		{
			return *UECurrencyFormattingRules;
		}
	}
	else
	{
		FScopeLock MapLock(&UEAlternateCurrencyFormattingRulesCS);

		TSharedPtr<const FDecimalNumberFormattingRules> FoundUEAlternateCurrencyFormattingRules = UEAlternateCurrencyFormattingRules.FindRef(SanitizedCurrencyCode);
		if (FoundUEAlternateCurrencyFormattingRules.IsValid())
		{
			return *FoundUEAlternateCurrencyFormattingRules;
		}
	}

	// Create a currency specific formatter (doesn't call CreateCurrencyFormat as we need a mutable instance)
	TSharedPtr<icu::DecimalFormat> CurrencyFormatterForCulture;
	{
		UErrorCode ICUStatus = U_ZERO_ERROR;
		CurrencyFormatterForCulture = MakeShareable(static_cast<icu::DecimalFormat*>(icu::NumberFormat::createCurrencyInstance(ICULocale, ICUStatus)));
		if (!ensureAlwaysMsgf(CurrencyFormatterForCulture, TEXT("Creating a currency format object failed using locale %s. Perhaps this locale has no data."), StringCast<TCHAR>(ICULocale.getName()).Get()))
		{
			ICUStatus = U_ZERO_ERROR;
			CurrencyFormatterForCulture = MakeShareable(static_cast<icu::DecimalFormat*>(icu::NumberFormat::createCurrencyInstance(GetInvariantLocale(), ICUStatus)));
			check(CurrencyFormatterForCulture);
		}
	}

	if (!bUseDefaultFormattingRules)
	{
		// Set the custom currency before we extract the data from the formatter
		icu::UnicodeString ICUCurrencyCode = ICUUtilities::ConvertString(SanitizedCurrencyCode);
		CurrencyFormatterForCulture->setCurrency(ICUCurrencyCode.getBuffer());
	}

	auto FixPrefixFormat = [](FString& InOutCurrencyPrefix)
	{
		const int32 PrefixLen = InOutCurrencyPrefix.Len();
		if (PrefixLen >= 3 && ICUUtilities::IsValidCurencyCodeCharacter(InOutCurrencyPrefix[PrefixLen - 1]) && ICUUtilities::IsValidCurencyCodeCharacter(InOutCurrencyPrefix[PrefixLen - 2]) && ICUUtilities::IsValidCurencyCodeCharacter(InOutCurrencyPrefix[PrefixLen - 3]))
		{
			InOutCurrencyPrefix.AppendChar(TEXT('\u00A0')); // No-break space
		}
	};

	auto FixSuffixFormat = [](FString& InOutCurrencySuffix)
	{
		const int32 SuffixLen = InOutCurrencySuffix.Len();
		if (SuffixLen >= 3 && ICUUtilities::IsValidCurencyCodeCharacter(InOutCurrencySuffix[0]) && ICUUtilities::IsValidCurencyCodeCharacter(InOutCurrencySuffix[1]) && ICUUtilities::IsValidCurencyCodeCharacter(InOutCurrencySuffix[2]))
		{
			InOutCurrencySuffix.InsertAt(0, TEXT('\u00A0')); // No-break space
		}
	};

	FDecimalNumberFormattingRules NewUECurrencyFormattingRules = ExtractNumberFormattingRulesFromICUDecimalFormatter(ICULocale, *CurrencyFormatterForCulture);
	
	// When a currency is from a different locale, it is common for ICU to disambiguate by prefixing or postfixing the numeric value
	// with ISO currency code. To ensure the readability in all cases we will add a no-break space between the currency code prefix/suffix 
	// and the numeric value, if the prefix/suffix would directly touch the numeric value (eg, to avoid a result like "JPY1 500").
	FixPrefixFormat(NewUECurrencyFormattingRules.PositivePrefixString);
	FixPrefixFormat(NewUECurrencyFormattingRules.NegativePrefixString);
	FixSuffixFormat(NewUECurrencyFormattingRules.PositiveSuffixString);
	FixSuffixFormat(NewUECurrencyFormattingRules.NegativeSuffixString);

	if (bUseDefaultFormattingRules)
	{
		// Check the pointer again in case another thread beat us to it
		{
			FScopeLock PtrLock(&UECurrencyFormattingRulesCS);

			if (!UECurrencyFormattingRules.IsValid())
			{
				UECurrencyFormattingRules = MakeShared<FDecimalNumberFormattingRules, ESPMode::ThreadSafe>(MoveTemp(NewUECurrencyFormattingRules));
			}
		}

		return *UECurrencyFormattingRules;
	}
	else
	{
		FScopeLock MapLock(&UEAlternateCurrencyFormattingRulesCS);

		// Find again in case another thread beat us to it
		TSharedPtr<const FDecimalNumberFormattingRules> FoundUEAlternateCurrencyFormattingRules = UEAlternateCurrencyFormattingRules.FindRef(SanitizedCurrencyCode);
		if (FoundUEAlternateCurrencyFormattingRules.IsValid())
		{
			return *FoundUEAlternateCurrencyFormattingRules;
		}

		FoundUEAlternateCurrencyFormattingRules = MakeShared<FDecimalNumberFormattingRules>(MoveTemp(NewUECurrencyFormattingRules));
		UEAlternateCurrencyFormattingRules.Add(SanitizedCurrencyCode, FoundUEAlternateCurrencyFormattingRules);
		return *FoundUEAlternateCurrencyFormattingRules;
	}
}

ETextPluralForm FICUCultureImplementation::GetPluralForm(int32 Val, const ETextPluralType PluralType) const
{
	checkf(Val >= 0, TEXT("GetPluralFormImpl requires a positive value"));

	const icu::PluralRules* ICUPluralRules = (PluralType == ETextPluralType::Cardinal) ? ICUCardinalPluralRules : ICUOrdinalPluralRules;
	const icu::UnicodeString ICUPluralFormTag = ICUPluralRules->select(Val);

	return ICUPluralFormToUE(ICUPluralFormTag);
}

ETextPluralForm FICUCultureImplementation::GetPluralForm(double Val, const ETextPluralType PluralType) const
{
	checkf(!FMath::IsNegativeOrNegativeZero(Val), TEXT("GetPluralFormImpl requires a positive value"));

	const icu::PluralRules* ICUPluralRules = (PluralType == ETextPluralType::Cardinal) ? ICUCardinalPluralRules : ICUOrdinalPluralRules;
	const icu::UnicodeString ICUPluralFormTag = ICUPluralRules->select(Val);

	return ICUPluralFormToUE(ICUPluralFormTag);
}

const TArray<ETextPluralForm>& FICUCultureImplementation::GetValidPluralForms(const ETextPluralType PluralType) const
{
	return (PluralType == ETextPluralType::Cardinal) ? UEAvailableCardinalPluralForms : UEAvailableOrdinalPluralForms;
}

#endif
