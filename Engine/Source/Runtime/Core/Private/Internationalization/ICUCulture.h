// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "Containers/UnrealString.h"
#include "Containers/Set.h"
#include "Containers/Map.h"
#include "Templates/SharedPointer.h"
#include "Internationalization/Text.h"
#include "Internationalization/Culture.h"
#include "Internationalization/CultureImplementation.h"

struct FDecimalNumberFormattingRules;

#if UE_ENABLE_ICU
THIRD_PARTY_INCLUDES_START
	#include <unicode/locid.h>
	#include <unicode/brkiter.h>
	#include <unicode/coll.h>
	#include <unicode/numfmt.h>
	#include <unicode/decimfmt.h>
PRAGMA_PUSH_PLATFORM_DEFAULT_PACKING
	#include <unicode/datefmt.h> // icu::Calendar can be affected by the non-standard packing UE4 uses, so force the platform default
PRAGMA_POP_PLATFORM_DEFAULT_PACKING
	#include <unicode/plurrule.h>
THIRD_PARTY_INCLUDES_END

struct FDecimalNumberFormattingRules;

class FInternationalization;

inline UColAttributeValue UEToICU(const ETextComparisonLevel::Type ComparisonLevel)
{
	UColAttributeValue Value;
	switch(ComparisonLevel)
	{
	case ETextComparisonLevel::Default:
		Value = UColAttributeValue::UCOL_DEFAULT;
		break;
	case ETextComparisonLevel::Primary:
		Value = UColAttributeValue::UCOL_PRIMARY;
		break;
	case ETextComparisonLevel::Secondary:
		Value = UColAttributeValue::UCOL_SECONDARY;
		break;
	case ETextComparisonLevel::Tertiary:
		Value = UColAttributeValue::UCOL_TERTIARY;
		break;
	case ETextComparisonLevel::Quaternary:
		Value = UColAttributeValue::UCOL_QUATERNARY;
		break;
	case ETextComparisonLevel::Quinary:
		Value = UColAttributeValue::UCOL_IDENTICAL;
		break;
	default:
		Value = UColAttributeValue::UCOL_DEFAULT;
		break;
	}
	return Value;
}

inline icu::DateFormat::EStyle UEToICU(const EDateTimeStyle::Type DateTimeStyle)
{
	icu::DateFormat::EStyle Value;
	switch(DateTimeStyle)
	{
	case EDateTimeStyle::Short:
		Value = icu::DateFormat::kShort;
		break;
	case EDateTimeStyle::Medium:
		Value = icu::DateFormat::kMedium;
		break;
	case EDateTimeStyle::Long:
		Value = icu::DateFormat::kLong;
		break;
	case EDateTimeStyle::Full:
		Value = icu::DateFormat::kFull;
		break;
	case EDateTimeStyle::Default:
		Value = icu::DateFormat::kDefault;
		break;
	default:
		Value = icu::DateFormat::kDefault;
		break;
	}
	return Value;
}

inline icu::DecimalFormat::ERoundingMode UEToICU(const ERoundingMode RoundingMode)
{
	icu::DecimalFormat::ERoundingMode Value;
	switch(RoundingMode)
	{
	case ERoundingMode::HalfToEven:
		Value = icu::DecimalFormat::ERoundingMode::kRoundHalfEven;
		break;
	case ERoundingMode::HalfFromZero:
		Value = icu::DecimalFormat::ERoundingMode::kRoundHalfUp;
		break;
	case ERoundingMode::HalfToZero:
		Value = icu::DecimalFormat::ERoundingMode::kRoundHalfDown;
		break;
	case ERoundingMode::FromZero:
		Value = icu::DecimalFormat::ERoundingMode::kRoundUp;
		break;
	case ERoundingMode::ToZero:
		Value = icu::DecimalFormat::ERoundingMode::kRoundDown;
		break;
	case ERoundingMode::ToNegativeInfinity:
		Value = icu::DecimalFormat::ERoundingMode::kRoundFloor;
		break;
	case ERoundingMode::ToPositiveInfinity:
		Value = icu::DecimalFormat::ERoundingMode::kRoundCeiling;
		break;
	default:
		Value = icu::DecimalFormat::ERoundingMode::kRoundHalfEven;
		break;
	}
	return Value;
}

inline ERoundingMode ICUToUE(const icu::DecimalFormat::ERoundingMode RoundingMode)
{
	ERoundingMode Value;
	switch(RoundingMode)
	{
	case icu::DecimalFormat::ERoundingMode::kRoundHalfEven:
		Value = ERoundingMode::HalfToEven;
		break;
	case icu::DecimalFormat::ERoundingMode::kRoundHalfUp:
		Value = ERoundingMode::HalfFromZero;
		break;
	case icu::DecimalFormat::ERoundingMode::kRoundHalfDown:
		Value = ERoundingMode::HalfToZero;
		break;
	case icu::DecimalFormat::ERoundingMode::kRoundUp:
		Value = ERoundingMode::FromZero;
		break;
	case icu::DecimalFormat::ERoundingMode::kRoundDown:
		Value = ERoundingMode::ToZero;
		break;
	case icu::DecimalFormat::ERoundingMode::kRoundFloor:
		Value = ERoundingMode::ToNegativeInfinity;
		break;
	case icu::DecimalFormat::ERoundingMode::kRoundCeiling:
		Value = ERoundingMode::ToPositiveInfinity;
		break;
	default:
		Value = ERoundingMode::HalfToEven;
		break;
	}
	return Value;
}

enum class EBreakIteratorType
{
	Grapheme,
	Word,
	Line,
	Sentence,
	Title
};

class FICUCultureImplementation : public ICultureImplementation
{
	friend class FCulture;
	friend class FText;
	friend class FTextChronoFormatter;
	friend class FICUBreakIteratorManager;

public:
	explicit FICUCultureImplementation(const FString& LocaleName);
	virtual ~FICUCultureImplementation() = default;

	//~ ICultureImplementation interface
	virtual FString GetDisplayName() const override;
	virtual FString GetEnglishName() const override;
	virtual int GetKeyboardLayoutId() const override;
	virtual int GetLCID() const override;
	virtual FString GetName() const override;
	virtual FString GetNativeName() const override;
	virtual FString GetUnrealLegacyThreeLetterISOLanguageName() const override;
	virtual FString GetThreeLetterISOLanguageName() const override;
	virtual FString GetTwoLetterISOLanguageName() const override;
	virtual FString GetNativeLanguage() const override;
	virtual FString GetNativeRegion() const override;
	virtual FString GetRegion() const override;
	virtual FString GetScript() const override;
	virtual FString GetVariant() const override;
	virtual bool IsRightToLeft() const override;

	// We run this in the open which is safe so long as we cannot create `FICUCultureImplementation` in the closed.
	AUTORTFM_OPEN virtual const FDecimalNumberFormattingRules& GetDecimalNumberFormattingRules() override;

	// We run this in the open which is safe so long as we cannot create `FICUCultureImplementation` in the closed.
	AUTORTFM_OPEN virtual const FDecimalNumberFormattingRules& GetPercentFormattingRules() override;

	// We run this in the open which is safe so long as we cannot create `FICUCultureImplementation` in the closed.
	AUTORTFM_OPEN virtual const FDecimalNumberFormattingRules& GetCurrencyFormattingRules(const FString& InCurrencyCode) override;
	virtual ETextPluralForm GetPluralForm(int32 Val, const ETextPluralType PluralType) const override;
	virtual ETextPluralForm GetPluralForm(double Val, const ETextPluralType PluralType) const override;
	virtual const TArray<ETextPluralForm>& GetValidPluralForms(const ETextPluralType PluralType) const override;

	static FString GetCanonicalName(const FString& Name, FInternationalization& I18N);

	TSharedRef<const icu::BreakIterator> GetBreakIterator(const EBreakIteratorType Type);
	TSharedRef<const icu::Collator, ESPMode::ThreadSafe> GetCollator(const ETextComparisonLevel::Type ComparisonLevel);
	TSharedRef<const icu::DecimalFormat, ESPMode::ThreadSafe> GetDateTimeDecimalFormat();
	TSharedRef<const icu::DateFormat, ESPMode::ThreadSafe> GetDateFormatter(const EDateTimeStyle::Type DateStyle, const FString& TimeZone);
	TSharedRef<const icu::DateFormat, ESPMode::ThreadSafe> GetTimeFormatter(const EDateTimeStyle::Type TimeStyle, const FString& TimeZone);
	TSharedRef<const icu::DateFormat, ESPMode::ThreadSafe> GetDateTimeFormatter(const EDateTimeStyle::Type DateStyle, const EDateTimeStyle::Type TimeStyle, const FString& TimeZone);
	TSharedRef<const icu::DateFormat, ESPMode::ThreadSafe> GetDateTimeFormatter(const FString& CustomPattern, const FString& TimeZone);

private:
	icu::Locale ICULocale;
	TSharedPtr<const icu::BreakIterator> ICUGraphemeBreakIterator;
	TSharedPtr<const icu::BreakIterator> ICUWordBreakIterator;
	TSharedPtr<const icu::BreakIterator> ICULineBreakIterator;
	TSharedPtr<const icu::BreakIterator> ICUSentenceBreakIterator;
	TSharedPtr<const icu::BreakIterator> ICUTitleBreakIterator;
	TSharedPtr<const icu::Collator, ESPMode::ThreadSafe> ICUCollator;

	TSharedPtr<const icu::DecimalFormat, ESPMode::ThreadSafe> ICUDateTimeDecimalFormat;
	TSharedPtr<const icu::DateFormat, ESPMode::ThreadSafe> ICUDateFormat;
	TSharedPtr<const icu::DateFormat, ESPMode::ThreadSafe> ICUTimeFormat;
	TSharedPtr<const icu::DateFormat, ESPMode::ThreadSafe> ICUDateTimeFormat;

	const icu::PluralRules* ICUCardinalPluralRules;
	const icu::PluralRules* ICUOrdinalPluralRules;

	TArray<ETextPluralForm> UEAvailableCardinalPluralForms;
	TArray<ETextPluralForm> UEAvailableOrdinalPluralForms;

	TSharedPtr<const FDecimalNumberFormattingRules, ESPMode::ThreadSafe> UEDecimalNumberFormattingRules;
	FCriticalSection UEDecimalNumberFormattingRulesCS;

	TSharedPtr<const FDecimalNumberFormattingRules, ESPMode::ThreadSafe> UEPercentFormattingRules;
	FCriticalSection UEPercentFormattingRulesCS;

	TSharedPtr<const FDecimalNumberFormattingRules, ESPMode::ThreadSafe> UECurrencyFormattingRules;
	FCriticalSection UECurrencyFormattingRulesCS;

	TMap<FString, TSharedPtr<const FDecimalNumberFormattingRules>> UEAlternateCurrencyFormattingRules;
	FCriticalSection UEAlternateCurrencyFormattingRulesCS;
};
#endif
