// Copyright Epic Games, Inc. All Rights Reserved.
#include "Internationalization/ICUInternationalization.h"
#include "HAL/FileManager.h"
#include "HAL/LowLevelMemTracker.h"
#include "Misc/ScopeLock.h"
#include "Misc/Paths.h"
#include "Internationalization/Culture.h"
#include "Internationalization/CultureFilter.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Cultures/LeetCulture.h"
#include "Internationalization/Cultures/KeysCulture.h"
#include "Stats/Stats.h"
#include "Misc/CoreStats.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/App.h"
#include "Algo/Transform.h"

#if UE_ENABLE_ICU

#include "Internationalization/ICURegex.h"
#include "Internationalization/ICUCulture.h"
#include "Internationalization/ICUUtilities.h"
#include "Internationalization/ICUBreakIterator.h"
THIRD_PARTY_INCLUDES_START
	#include <unicode/locid.h>
PRAGMA_PUSH_PLATFORM_DEFAULT_PACKING
	#include <unicode/timezone.h> // icu::Calendar can be affected by the non-standard packing UE4 uses, so force the platform default
PRAGMA_POP_PLATFORM_DEFAULT_PACKING
	#include <unicode/uclean.h>
	#include <unicode/udata.h>
THIRD_PARTY_INCLUDES_END

DEFINE_LOG_CATEGORY_STATIC(LogICUInternationalization, Log, All);

static_assert(sizeof(UChar) == 2, "UChar (from ICU) is assumed to always be 2-bytes!");
static_assert(PLATFORM_LITTLE_ENDIAN, "ICU data is only built for little endian platforms. You'll need to rebuild the data for your platform and update this code!");


#if WITH_ICU_V78 && PLATFORM_WINDOWS
	static_assert(sizeof(icu::Calendar) == 264, "icu::Calendar should be 264-bytes! Ensure relevant includes are wrapped in PRAGMA_PUSH_PLATFORM_DEFAULT_PACKING and PRAGMA_POP_PLATFORM_DEFAULT_PACKING.");
#elif WITH_ICU_V64 && PLATFORM_WINDOWS
	static_assert(sizeof(icu::Calendar) == 616, "icu::Calendar should be 616-bytes! Ensure relevant includes are wrapped in PRAGMA_PUSH_PLATFORM_DEFAULT_PACKING and PRAGMA_POP_PLATFORM_DEFAULT_PACKING.");
#endif

namespace
{
	struct FICUOverrides
	{
#if STATS
		static int64 BytesInUseCount;
		static int64 CachedBytesInUseCount;
#endif

		static void* U_CALLCONV Malloc(const void* context, size_t size)
		{
			LLM_SCOPE_BYNAME(TEXT("Localization/ICU"));
			void* Result = FMemory::Malloc(size);
#if STATS
			BytesInUseCount += FMemory::GetAllocSize(Result);
			if(FThreadStats::IsThreadingReady() && CachedBytesInUseCount != BytesInUseCount)
			{
				// The amount of startup stats messages for STAT_MemoryICUTotalAllocationSize is about 700k
				// It needs to be replaced with something like this
				//  FEngineLoop.Tick.{ 
				//		FPlatformMemory::UpdateStats();
				//		
				//	And called once per frame.
				//SET_MEMORY_STAT(STAT_MemoryICUTotalAllocationSize, BytesInUseCount);
				CachedBytesInUseCount = BytesInUseCount;
			}
#endif
			return Result;
		}

		static void* U_CALLCONV Realloc(const void* context, void* mem, size_t size)
		{
			LLM_SCOPE_BYNAME(TEXT("Localization/ICU"));
			return FMemory::Realloc(mem, size);
		}

		static void U_CALLCONV Free(const void* context, void* mem)
		{
#if STATS
			BytesInUseCount -= FMemory::GetAllocSize(mem);
			if(FThreadStats::IsThreadingReady() && CachedBytesInUseCount != BytesInUseCount)
			{
				//SET_MEMORY_STAT(STAT_MemoryICUTotalAllocationSize, BytesInUseCount);
				CachedBytesInUseCount = BytesInUseCount;
			}
#endif
			FMemory::Free(mem);
		}
	};

#if STATS
	int64 FICUOverrides::BytesInUseCount = 0;
	int64 FICUOverrides::CachedBytesInUseCount = 0;
#endif
}

FICUInternationalization::FICUInternationalization(FInternationalization* const InI18N)
	: I18N(InI18N)
{

}

bool FICUInternationalization::Initialize()
{
	UErrorCode ICUStatus = U_ZERO_ERROR;

#if NEEDS_ICU_DLLS
	LoadDLLs();
#endif /*IS_PROGRAM || !IS_MONOLITHIC*/

	u_setMemoryFunctions(nullptr, &FICUOverrides::Malloc, &FICUOverrides::Realloc, &FICUOverrides::Free, &ICUStatus);

	const FString DataDirectoryRelativeToContent = "Internationalization";
	const FString PotentialDataDirectories[] =
	{
		FPaths::ProjectContentDir() / DataDirectoryRelativeToContent, // Try game content directory.
		FPaths::EngineContentDir() / DataDirectoryRelativeToContent, // Try engine content directory.
	};

	ICUDataDirectory.Reset();
	for (const FString& PotentialDataDirectory : PotentialDataDirectories)
	{
		if (FPaths::DirectoryExists(PotentialDataDirectory))
		{
			u_setDataDirectory(StringCast<char>(*PotentialDataDirectory).Get());
#if WITH_ICU_V78
			ICUDataDirectory = PotentialDataDirectory / TEXT("icudt78l") / TEXT(""); // We include the sub-folder here as it prevents ICU I/O requests outside of this folder
#elif WITH_ICU_V64
			ICUDataDirectory = PotentialDataDirectory / TEXT("icudt64l") / TEXT(""); // We include the sub-folder here as it prevents ICU I/O requests outside of this folder
#else
			ICUDataDirectory = PotentialDataDirectory / TEXT("icudt53l") / TEXT(""); // We include the sub-folder here as it prevents ICU I/O requests outside of this folder
#endif
			// Normalize so the path is in the same form (lowercased on Windows) that OpenDataFile
			// produces via FPaths::NormalizeFilename — ensures StartsWith comparisons don't fail
			// due to casing mismatches on case-insensitive platforms.
			FPaths::NormalizeFilename(ICUDataDirectory);
			break;
		}
	}

	if (ICUDataDirectory.IsEmpty())
	{
		auto GetPrioritizedDataDirectoriesString = [&]() -> FString
		{
			FString Result;
			for (const auto& DataDirectoryString : PotentialDataDirectories)
			{
				if (!Result.IsEmpty())
				{
					Result += TEXT("\n");
				}
				Result += DataDirectoryString;
			}
			return Result;
		};

		UE_LOGF(LogICUInternationalization, Fatal, "ICU data directory was not discovered:\n%ls", *GetPrioritizedDataDirectoriesString());
	}

	udata_setFileAccess(UDATA_FILES_FIRST, &ICUStatus); // We always need to load loose ICU data files
	u_setDataFileFunctions(this, &FICUInternationalization::OpenDataFile, &FICUInternationalization::CloseDataFile, &ICUStatus);
	u_init(&ICUStatus);
	checkf(U_SUCCESS(ICUStatus), TEXT("Failed to open ICUInternationalization data file, missing or corrupt?"));

	FICURegexManager::Create();
	FICUBreakIteratorManager::Create();

	InitializeAvailableCultures();

	bHasInitializedCultureMappings = false;
	ConditionalInitializeCultureMappings();

	bHasInitializedAllowedCultures = false;
	ConditionalInitializeAllowedCultures();

	I18N->InvariantCulture = FindOrMakeCanonizedCulture(TEXT("en-US-POSIX"), EAllowDefaultCultureFallback::No);
	if (!I18N->InvariantCulture.IsValid())
	{
		I18N->InvariantCulture = FindOrMakeCanonizedCulture(FString(), EAllowDefaultCultureFallback::Yes);
	}
	I18N->DefaultLanguage = FindOrMakeCulture(FPlatformMisc::GetDefaultLanguage(), EAllowDefaultCultureFallback::Yes);
	if (const FString& DefaultLocaleName = FCultureImplementation::GetCanonicalName(FPlatformMisc::GetDefaultLocale(), *I18N);
		AllAvailableCulturesMap.Contains(DefaultLocaleName))
	{
		I18N->DefaultLocale = FindOrMakeCanonizedCulture(DefaultLocaleName, EAllowDefaultCultureFallback::Yes);
	}
	else
	{
		UE_LOGF(LogICUInternationalization, Log, "OS requested locale '%ls' is not supported. Using the OS requested language of '%ls' as the locale.", *DefaultLocaleName, *I18N->DefaultLanguage->GetName());
		I18N->DefaultLocale = I18N->DefaultLanguage;
	}
	I18N->CurrentLanguage = I18N->DefaultLanguage;
	I18N->CurrentLocale = I18N->DefaultLocale;

	HandleLanguageChanged(I18N->CurrentLanguage.ToSharedRef());

#if ENABLE_LOC_TESTING
	I18N->AddCustomCulture(MakeShared<FLeetCulture>(I18N->InvariantCulture.ToSharedRef()));
	I18N->AddCustomCulture(MakeShared<FKeysCulture>(I18N->InvariantCulture.ToSharedRef()));
#endif

	InitializeTimeZone();
	InitializeInvariantGregorianCalendar();

	return U_SUCCESS(ICUStatus) ? true : false;
}

void FICUInternationalization::Terminate()
{
	InvariantGregorianCalendar.Reset();

	FICURegexManager::Destroy();
	FICUBreakIteratorManager::Destroy();
	CachedCultures.Empty();

	u_cleanup();

	FScopeLock Lock(&PathToCachedFileDataMapCS);
	for (auto& PathToCachedFileDataPair : PathToCachedFileDataMap)
	{
		UE_LOGF(LogICUInternationalization, Warning, "ICU data file '%ls' (ref count %d) was still referenced after ICU shutdown. This will likely lead to a crash.", *PathToCachedFileDataPair.Key, PathToCachedFileDataPair.Value.ReferenceCount);
	}
	PathToCachedFileDataMap.Empty();

#if NEEDS_ICU_DLLS
	UnloadDLLs();
#endif //IS_PROGRAM || !IS_MONOLITHIC
}

#if NEEDS_ICU_DLLS
void FICUInternationalization::LoadDLLs()
{
	// The base directory for ICU binaries is consistent on all platforms.
	FString ICUBinariesRoot = FPaths::EngineDir() / TEXT("Binaries") / TEXT("ThirdParty") / TEXT("ICU") / TEXT("icu4c-53_1");

#if PLATFORM_WINDOWS
	const FString PlatformFolderName = "Win64";

#if _MSC_VER >= 1900
	const FString VSVersionFolderName = "VS2015";
#else
	#error "FICUInternationalization::LoadDLLs - Unknown _MSC_VER! Please update this code for this version of MSVC."
#endif //_MSVC_VER

	// Windows requires support for 32/64 bit and different MSVC runtimes.
	const FString TargetSpecificPath = ICUBinariesRoot / PlatformFolderName / VSVersionFolderName;

	// Windows libraries use a specific naming convention.
	FString LibraryNameStems[] =
	{
		TEXT("dt"),   // Data
		TEXT("uc"),   // Unicode Common
		TEXT("in"),   // Internationalization
		TEXT("le"),   // Layout Engine
		TEXT("lx"),   // Layout Extensions
		TEXT("io")	// Input/Output
	};
#else
	// Non-Windows libraries use a consistent naming convention.
	FString LibraryNameStems[] =
	{
		TEXT("data"),   // Data
		TEXT("uc"),   // Unicode Common
		TEXT("i18n"),   // Internationalization
		TEXT("le"),   // Layout Engine
		TEXT("lx"),   // Layout Extensions
		TEXT("io")	// Input/Output
	};
#if PLATFORM_LINUX
	const FString TargetSpecificPath = ICUBinariesRoot / TEXT("Linux") / TEXT("x86_64-unknown-linux-gnu");
#elif PLATFORM_MAC
	const FString TargetSpecificPath = ICUBinariesRoot / TEXT("Mac");
#endif //PLATFORM_*
#endif //PLATFORM_*

#if UE_BUILD_DEBUG && !defined(NDEBUG)
	const FString LibraryNamePostfix = "d";
#else
	const FString LibraryNamePostfix;
#endif //DEBUG

	for(FString Stem : LibraryNameStems)
	{
#if PLATFORM_WINDOWS
		FString LibraryName = "icu" + Stem + LibraryNamePostfix + "53" "." "dll";
#elif PLATFORM_LINUX
		FString LibraryName = "lib" "icu" + Stem + LibraryNamePostfix + ".53.1" "." "so";
#elif PLATFORM_MAC
		FString LibraryName = "lib" "icu" + Stem + ".53.1" + LibraryNamePostfix + "." "dylib";
#endif //PLATFORM_*

		const FString DllPath = TargetSpecificPath / LibraryName;
		void* DLLHandle = FPlatformProcess::GetDllHandle(*DllPath);
		checkf(DLLHandle != nullptr, TEXT("GetDllHandle failed to load: '%s'"), *DllPath);
		DLLHandles.Add(DLLHandle);
	}
}

void FICUInternationalization::UnloadDLLs()
{
	for( const auto DLLHandle : DLLHandles )
	{
		FPlatformProcess::FreeDllHandle(DLLHandle);
	}
	DLLHandles.Reset();
}
#endif // NEEDS_ICU_DLLS

#if STATS
namespace
{
	int64 DataFileBytesInUseCount = 0;
	int64 CachedDataFileBytesInUseCount = 0;
}
#endif

void FICUInternationalization::LoadAllCultureData()
{
	for (const FICUCultureData& CultureData : AllAvailableCultures)
	{
		FindOrMakeCanonizedCulture(CultureData.Name, EAllowDefaultCultureFallback::No);
	}
}

void FICUInternationalization::InitializeAvailableCultures()
{
	// Build up the data about all available locales
	int32_t LocaleCount = 0;
	const icu::Locale* const AvailableLocales = icu::Locale::getAvailableLocales(LocaleCount);

	AllAvailableCultures.Reserve(LocaleCount);
	AllAvailableCulturesMap.Reserve(LocaleCount);
	AllAvailableLanguagesToSubCulturesMap.Reserve(LocaleCount / 2);

	auto AppendCultureData = [&](const FString& InCultureName, const FString& InLanguageCode, const FString& InScriptCode, const FString& InCountryCode)
	{
		if (AllAvailableCulturesMap.Contains(InCultureName))
		{
			return;
		}

		const int32 CultureDataIndex = AllAvailableCultures.Num();
		AllAvailableCulturesMap.Add(InCultureName, CultureDataIndex);

		TArray<int32>& CulturesForLanguage = AllAvailableLanguagesToSubCulturesMap.FindOrAdd(InLanguageCode);
		CulturesForLanguage.Add(CultureDataIndex);

		FICUCultureData& CultureData = AllAvailableCultures.AddDefaulted_GetRef();
		CultureData.Name = InCultureName;
		CultureData.LanguageCode = InLanguageCode;
		CultureData.ScriptCode = InScriptCode;
		CultureData.CountryCode = InCountryCode;
	};

	for (int32 i = 0; i < LocaleCount; ++i)
	{
		const icu::Locale& Locale = AvailableLocales[i];

		const FString LanguageCode = Locale.getLanguage();
		const FString ScriptCode = Locale.getScript();
		const FString CountryCode = Locale.getCountry();

		// AvailableLocales doesn't always contain all variations of a culture, so we try and add them all here
		// This allows the culture script look-up in GetPrioritizedCultureNames to work without having to load up culture data most of the time
		AppendCultureData(LanguageCode, LanguageCode, FString(), FString());
		if (!CountryCode.IsEmpty())
		{
			AppendCultureData(FCulture::CreateCultureName(LanguageCode, FString(), CountryCode), LanguageCode, FString(), CountryCode);
		}
		if (!ScriptCode.IsEmpty())
		{
			AppendCultureData(FCulture::CreateCultureName(LanguageCode, ScriptCode, FString()), LanguageCode, ScriptCode, FString());
		}
		if (!ScriptCode.IsEmpty() && !CountryCode.IsEmpty())
		{
			AppendCultureData(FCulture::CreateCultureName(LanguageCode, ScriptCode, CountryCode), LanguageCode, ScriptCode, CountryCode);
		}
	}

	// getAvailableLocales doesn't always cover all languages that ICU supports, so we spin that list too and add any that were missed
	// Note: getISOLanguages returns the start of an array of const char* null-terminated strings, with a null string signifying the end of the array
	for (const char* const* AvailableLanguages = icu::Locale::getISOLanguages(); *AvailableLanguages; ++AvailableLanguages)
	{
		// Only care about 2-letter codes
		const char* AvailableLanguage = *AvailableLanguages;
		if (AvailableLanguage[2] == 0)
		{
			FString LanguageCode = AvailableLanguage;
			LanguageCode.ToLowerInline();

			AppendCultureData(LanguageCode, LanguageCode, FString(), FString());
		}
	}

	// Also add our invariant culture if it wasn't found when processing the ICU locales
	AppendCultureData(TEXT("en-US-POSIX"), TEXT("en"), FString(), TEXT("US-POSIX"));

	AllAvailableCultures.Shrink();
	AllAvailableCulturesMap.Shrink();
	AllAvailableLanguagesToSubCulturesMap.Shrink();
}

TArray<FString> LoadInternationalizationConfigArray(const TCHAR* InKey)
{
	check(GConfig && GConfig->IsReadyForUse());

	const bool ShouldLoadEditor = GIsEditor;
	const bool ShouldLoadGame = FApp::IsGame();

	TArray<FString> FinalArray;
	GConfig->GetArray(TEXT("Internationalization"), InKey, FinalArray, GEngineIni);

	if (ShouldLoadEditor)
	{
		TArray<FString> EditorArray;
		GConfig->GetArray(TEXT("Internationalization"), InKey, EditorArray, GEditorIni);
		FinalArray.Append(MoveTemp(EditorArray));
	}

	if (ShouldLoadGame)
	{
		TArray<FString> GameArray;
		GConfig->GetArray(TEXT("Internationalization"), InKey, GameArray, GGameIni);
		FinalArray.Append(MoveTemp(GameArray));
	}

	return FinalArray;
}

void FICUInternationalization::ConditionalInitializeCultureMappings()
{
	if (bHasInitializedCultureMappings || !GConfig || !GConfig->IsReadyForUse())
	{
		return;
	}

	bHasInitializedCultureMappings = true;

	const TArray<FString> CultureMappingsArray = LoadInternationalizationConfigArray(TEXT("CultureMappings"));

	// An array of semicolon separated mapping entries: SourceCulture;DestCulture
	CultureMappings.Reserve(CultureMappingsArray.Num());
	for (const FString& CultureMappingStr : CultureMappingsArray)
	{
		FString SourceCulture;
		FString DestCulture;
		if (CultureMappingStr.Split(TEXT(";"), &SourceCulture, &DestCulture, ESearchCase::CaseSensitive))
		{
			if (AllAvailableCulturesMap.Contains(DestCulture))
			{
				CultureMappings.Add(MoveTemp(SourceCulture), MoveTemp(DestCulture));
			}
			else
			{
				UE_LOGF(LogICUInternationalization, Warning, "Culture mapping '%ls' contains an unknown culture and has been ignored.", *CultureMappingStr);
			}
		}
	}
	CultureMappings.Compact();
}

void FICUInternationalization::ConditionalInitializeAllowedCultures()
{
	if (bHasInitializedAllowedCultures || !GConfig || !GConfig->IsReadyForUse())
	{
		return;
	}

	bHasInitializedAllowedCultures = true;

	TSet<FString> AvailableCulturesSet;
	AvailableCulturesSet.Reserve(AllAvailableCulturesMap.Num());
	Algo::Transform(AllAvailableCulturesMap, AvailableCulturesSet, &TTuple<FString, int32>::Key);

	AllowedCulturesFilter = MakePimpl<FCultureFilter>(&AvailableCulturesSet);
}

bool FICUInternationalization::IsCultureRemapped(const FString& Name, FString* OutMappedCulture)
{
	// Make sure we've loaded the culture mappings (the config system may not have been available when we were first initialized)
	ConditionalInitializeCultureMappings();

	// Check to see if the culture has been re-mapped
	const FString* const MappedCulture = CultureMappings.Find(Name);
	if (MappedCulture && OutMappedCulture)
	{
		*OutMappedCulture = *MappedCulture;
	}

	return MappedCulture != nullptr;
}

bool FICUInternationalization::IsCultureAllowed(const FString& Name)
{
	// Make sure we've loaded the allowed cultures lists (the config system may not have been available when we were first initialized)
	ConditionalInitializeAllowedCultures();

	return AllowedCulturesFilter->IsCultureAllowed(Name);
}

void FICUInternationalization::RefreshCultureDisplayNames(const TArray<FString>& InPrioritizedDisplayCultureNames)
{
	// Update the cached display names in any existing cultures
	FScopeLock Lock(&CachedCulturesCS);
	for (const auto& CachedCulturePair : CachedCultures)
	{
		CachedCulturePair.Value->RefreshCultureDisplayNames(InPrioritizedDisplayCultureNames);
	}
}

void FICUInternationalization::RefreshCachedConfigData()
{
	bHasInitializedCultureMappings = false;
	CultureMappings.Reset();
	ConditionalInitializeCultureMappings();

	bHasInitializedAllowedCultures = false;
	AllowedCulturesFilter.Reset();
	ConditionalInitializeAllowedCultures();
}

void FICUInternationalization::HandleLanguageChanged(const FCultureRef InNewLanguage)
{
	UErrorCode ICUStatus = U_ZERO_ERROR;
	uloc_setDefault(StringCast<char>(*InNewLanguage->GetName()).Get(), &ICUStatus);

	CachedPrioritizedDisplayCultureNames = InNewLanguage->GetPrioritizedParentCultureNames();

	// Update the cached display names in any existing cultures
	FScopeLock Lock(&CachedCulturesCS);
	for (const auto& CachedCulturePair : CachedCultures)
	{
		CachedCulturePair.Value->RefreshCultureDisplayNames(CachedPrioritizedDisplayCultureNames, /*bFullRefresh*/false);
	}
}

void FICUInternationalization::GetCultureNames(TArray<FString>& CultureNames) const
{
	CultureNames.Reset(AllAvailableCultures.Num() + I18N->CustomCultures.Num());
	for (const FICUCultureData& CultureData : AllAvailableCultures)
	{
		CultureNames.Add(CultureData.Name);
	}
	for (const FCultureRef& CustomCulture : I18N->CustomCultures)
	{
		CultureNames.Add(CustomCulture->GetName());
	}
}

TArray<FString> FICUInternationalization::GetPrioritizedCultureNames(const FString& Name)
{
	auto PopulateCultureData = [&](const FString& InCultureName, FICUCultureData& OutCultureData) -> bool
	{
		// First, try and find the data in the map (although it seems that not all data is in here)
		const int32* CultureDataIndexPtr = AllAvailableCulturesMap.Find(InCultureName);
		if (CultureDataIndexPtr)
		{
			OutCultureData = AllAvailableCultures[*CultureDataIndexPtr];
			return true;
		}
		
		// Failing that, try and find the culture directly (this will cause its resource data to be loaded)
		FCulturePtr Culture = FindOrMakeCanonizedCulture(InCultureName, EAllowDefaultCultureFallback::No);
		if (Culture.IsValid())
		{
			OutCultureData.Name = Culture->GetName();
			OutCultureData.LanguageCode = Culture->GetTwoLetterISOLanguageName();
			OutCultureData.ScriptCode = Culture->GetScript();
			OutCultureData.CountryCode = Culture->GetRegion();
			return true;
		}

		return false;
	};

	// Apply any culture remapping
	FString GivenCulture = FCultureImplementation::GetCanonicalName(Name, *I18N);
	IsCultureRemapped(Name, &GivenCulture);

	TArray<FString> PrioritizedCultureNames;

	FICUCultureData GivenCultureData;
	if (PopulateCultureData(GivenCulture, GivenCultureData))
	{
		// If we have a culture without a script, but with a country code, we can try and work out the script for the country code by enumerating all of the available cultures
		// and looking for a matching culture with a script set (eg, "zh-CN" would find "zh-Hans-CN")
		TArray<FICUCultureData> ParentCultureData;
		if (GivenCultureData.ScriptCode.IsEmpty() && !GivenCultureData.CountryCode.IsEmpty())
		{
			const TArray<int32>* const CulturesForLanguage = AllAvailableLanguagesToSubCulturesMap.Find(GivenCultureData.LanguageCode);
			if (CulturesForLanguage)
			{
				for (const int32 CultureIndex : *CulturesForLanguage)
				{
					const FICUCultureData& CultureData = AllAvailableCultures[CultureIndex];
					if (!CultureData.ScriptCode.IsEmpty() && GivenCultureData.CountryCode == CultureData.CountryCode)
					{
						ParentCultureData.Add(CultureData);
					}
				}
			}
		}
		
		if (ParentCultureData.Num() == 0)
		{
			ParentCultureData.Add(GivenCultureData);
		}

		TArray<FICUCultureData> PrioritizedCultureData;

		PrioritizedCultureData.Reserve(ParentCultureData.Num() * 3);
		for (const FICUCultureData& CultureData : ParentCultureData)
		{
			const TArray<FString> PrioritizedParentCultures = FCulture::GetPrioritizedParentCultureNames(CultureData.LanguageCode, CultureData.ScriptCode, CultureData.CountryCode);
			for (const FString& PrioritizedParentCultureName : PrioritizedParentCultures)
			{
				FICUCultureData PrioritizedParentCultureData;
				if (PopulateCultureData(PrioritizedParentCultureName, PrioritizedParentCultureData))
				{
					PrioritizedCultureData.AddUnique(PrioritizedParentCultureData);
				}
			}
		}

		// Sort the cultures by their priority
		// Special case handling for the ambiguity of Hong Kong and Macau supporting both Traditional and Simplified Chinese (prefer Traditional)
		const bool bPreferTraditionalChinese = GivenCultureData.CountryCode == TEXT("HK") || GivenCultureData.CountryCode == TEXT("MO");
		PrioritizedCultureData.Sort([bPreferTraditionalChinese](const FICUCultureData& DataOne, const FICUCultureData& DataTwo) -> bool
		{
			const int32 DataOneSortWeight = (DataOne.CountryCode.IsEmpty() ? 0 : 4) + (DataOne.ScriptCode.IsEmpty() ? 0 : 2) + (bPreferTraditionalChinese && DataOne.ScriptCode == TEXT("Hant") ? 1 : 0);
			const int32 DataTwoSortWeight = (DataTwo.CountryCode.IsEmpty() ? 0 : 4) + (DataTwo.ScriptCode.IsEmpty() ? 0 : 2) + (bPreferTraditionalChinese && DataTwo.ScriptCode == TEXT("Hant") ? 1 : 0);
			return DataOneSortWeight >= DataTwoSortWeight;
		});

		PrioritizedCultureNames.Reserve(PrioritizedCultureData.Num());
		for (const FICUCultureData& CultureData : PrioritizedCultureData)
		{
			// Remove any cultures that are explicitly disallowed
			if (IsCultureAllowed(CultureData.Name))
			{
				PrioritizedCultureNames.Add(CultureData.Name);
			}
		}
	}

	// If we have no cultures, fallback to using English
	if (PrioritizedCultureNames.Num() == 0)
	{
		PrioritizedCultureNames.Add(TEXT("en"));
	}

	return PrioritizedCultureNames;
}

FCulturePtr FICUInternationalization::GetCulture(const FString& Name)
{
	return FindOrMakeCulture(Name, EAllowDefaultCultureFallback::No);
}

FCulturePtr FICUInternationalization::FindOrMakeCulture(const FString& Name, const EAllowDefaultCultureFallback AllowDefaultFallback)
{
	return FindOrMakeCanonizedCulture(FCultureImplementation::GetCanonicalName(Name, *I18N), AllowDefaultFallback);
}

FCulturePtr FICUInternationalization::FindOrMakeCanonizedCulture(const FString& Name, const EAllowDefaultCultureFallback AllowDefaultFallback)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FICUInternationalization::FindOrMakeCanonizedCulture);

	// Find the cached culture.
	{
		FScopeLock Lock(&CachedCulturesCS);
		FCultureRef* FoundCulture = CachedCultures.Find(Name);
		if (FoundCulture)
		{
			return *FoundCulture;
		}
	}

	// If no cached culture is found, try to make one.
	FCulturePtr NewCulture;

	// Is this a custom culture?
	if (FCulturePtr CustomCulture = I18N->GetCustomCulture(Name))
	{
		NewCulture = CustomCulture;
	}
	// Is this in our list of available cultures?
	else if (AllAvailableCulturesMap.Contains(Name))
	{
		NewCulture = FCulture::Create(MakeUnique<FICUCultureImplementation>(Name));
	}
	else
	{
		// We need to use a resource load in order to get the correct culture
		UErrorCode ICUStatus = U_ZERO_ERROR;
		if (UResourceBundle* ICUResourceBundle = ures_open(nullptr, StringCast<char>(*Name).Get(), &ICUStatus))
		{
			if (ICUStatus != U_USING_DEFAULT_WARNING || AllowDefaultFallback == EAllowDefaultCultureFallback::Yes)
			{
				NewCulture = FCulture::Create(MakeUnique<FICUCultureImplementation>(Name));
			}
			ures_close(ICUResourceBundle);
		}
	}

	if (NewCulture.IsValid())
	{
		// Ensure the display name is up-to-date
		NewCulture->RefreshCultureDisplayNames(CachedPrioritizedDisplayCultureNames, /*bFullRefresh*/false);

		{
			FScopeLock Lock(&CachedCulturesCS);
			CachedCultures.Add(Name, NewCulture.ToSharedRef());
		}
	}

	return NewCulture;
}

void FICUInternationalization::InitializeTimeZone()
{
	const FString TimeZoneId = FPlatformMisc::GetTimeZoneId();

	icu::TimeZone* ICUDefaultTz = TimeZoneId.IsEmpty() ? icu::TimeZone::createDefault() : icu::TimeZone::createTimeZone(ICUUtilities::ConvertString(TimeZoneId));
	icu::TimeZone::adoptDefault(ICUDefaultTz);

	const int32 DefaultTzOffsetMinutes = ICUDefaultTz->getRawOffset() / 60000;
	const int32 RawOffsetHours = DefaultTzOffsetMinutes / 60;
	const int32 RawOffsetMinutes = DefaultTzOffsetMinutes % 60;
	UE_LOGF(LogICUInternationalization, Log, "ICU TimeZone Detection - Raw Offset: %+d:%02d, Platform Override: '%ls'", RawOffsetHours, RawOffsetMinutes, *TimeZoneId);
}

void FICUInternationalization::InitializeInvariantGregorianCalendar()
{
	UErrorCode ICUStatus = U_ZERO_ERROR;
	InvariantGregorianCalendar = MakeUnique<icu::GregorianCalendar>(ICUStatus);
	if (InvariantGregorianCalendar)
	{
		InvariantGregorianCalendar->setTimeZone(icu::TimeZone::getUnknown());
	}
}

UDate FICUInternationalization::UEDateTimeToICUDate(const FDateTime& DateTime)
{
	UDate ICUDate = 0;

	if (InvariantGregorianCalendar)
	{
		// UE4 and ICU have a different time scale for pre-Gregorian dates, so we can't just use the UNIX timestamp from the UE4 DateTime
		// Instead we have to explode the UE4 DateTime into its component parts, and then use an ICU GregorianCalendar (set to the "unknown" 
		// timezone so it doesn't apply any adjustment to the time) to reconstruct the DateTime as an ICU UDate in the correct scale
		int32 Year, Month, Day;
		DateTime.GetDate(Year, Month, Day);
		const int32 Hour = DateTime.GetHour();
		const int32 Minute = DateTime.GetMinute();
		const int32 Second = DateTime.GetSecond();

		{
			FScopeLock Lock(&InvariantGregorianCalendarCS);

			InvariantGregorianCalendar->set(Year, Month - 1, Day, Hour, Minute, Second);

			UErrorCode ICUStatus = U_ZERO_ERROR;
			ICUDate = InvariantGregorianCalendar->getTime(ICUStatus);
		}
	}
	else
	{
		// This method is less accurate (see the comment above), but will work well enough if an ICU GregorianCalendar isn't available
		const int64 UnixTimestamp = DateTime.ToUnixTimestamp();
		ICUDate = static_cast<UDate>(static_cast<double>(UnixTimestamp) * U_MILLIS_PER_SECOND);
	}

	return ICUDate;
}

/**
 * Callback installed via u_setDataFileFunctions(), replacing ICU's default fopen()/mmap() data loading.
 * ICU's native file I/O bypasses UE's virtual filesystem and is unavailable on some platforms; this
 * implementation routes all ICU data file access through IFileManager and caches loaded files by path.
 *
 * OutLength is new in ICU 78. initAliasData() (ucnv_io.cpp) now validates each loaded file's size via
 * udata_getLength() before use. Without a valid length, udata_getLength() returns -1 and u_init() fails
 * with U_INVALID_FORMAT_ERROR. ICU's native paths populate UDataMemory::length from the OS file size
 * automatically; the callback path had no equivalent until this parameter was added.
 */
#if WITH_ICU_V78
UBool FICUInternationalization::OpenDataFile(const void* InContext, void** OutFileContext, void** OutContents, const char* InPath, int32_t* OutLength)
#else
UBool FICUInternationalization::OpenDataFile(const void* InContext, void** OutFileContext, void** OutContents, const char* InPath)
#endif
{
	LLM_SCOPE_BYNAME(TEXT("Localization/ICU"));

	FICUInternationalization* This = (FICUInternationalization*)InContext;
	check(This);

	FString PathStr = StringCast<TCHAR>(InPath).Get();
	FPaths::NormalizeFilename(PathStr);

	// Skip requests for anything outside the ICU data directory
	const bool bIsWithinDataDirectory = PathStr.StartsWith(This->ICUDataDirectory);
	if (!bIsWithinDataDirectory)
	{
		*OutFileContext = nullptr;
		*OutContents = nullptr;
		return false;
	}

	FScopeLock Lock(&This->PathToCachedFileDataMapCS);
	FICUCachedFileData* CachedFileData = This->PathToCachedFileDataMap.Find(PathStr);

	// If there's no file context, we might have to load the file.
	if (!CachedFileData)
	{
#if !UE_BUILD_SHIPPING
		FScopedLoadingState ScopedLoadingState(*PathStr);
#endif

		// Attempt to load the file.
		FArchive* FileAr = IFileManager::Get().CreateFileReader(*PathStr);
		if (FileAr)
		{
			const int64 FileSize = FileAr->TotalSize();

			// Create file data.
			CachedFileData = &This->PathToCachedFileDataMap.Emplace(PathStr, FICUCachedFileData(FileSize));

			// Load file into buffer.
			FileAr->Serialize(CachedFileData->Buffer, FileSize);
			delete FileAr;

			// Stat tracking.
#if STATS
			DataFileBytesInUseCount += FMemory::GetAllocSize(CachedFileData->Buffer);
			if (FThreadStats::IsThreadingReady() && CachedDataFileBytesInUseCount != DataFileBytesInUseCount)
			{
				SET_MEMORY_STAT(STAT_MemoryICUDataFileAllocationSize, DataFileBytesInUseCount);
				CachedDataFileBytesInUseCount = DataFileBytesInUseCount;
			}
#endif
				This->TotalCachedFileDataBytes += FileSize;
				UE_LOGF(LogICUInternationalization, Verbose, "Loaded: %ls (%.1f KB, %d files cached, %.1f KB total)",
					*PathStr, static_cast<double>(FileSize) / 1024.0, This->PathToCachedFileDataMap.Num(), static_cast<double>(This->TotalCachedFileDataBytes) / 1024.0);
		}
	}

	if (CachedFileData)
	{
		// Add a reference, either the initial one or an additional one.
		++CachedFileData->ReferenceCount;

		// Use the file path as the context, so we can look up the cached file data later and decrement its reference count.
		*OutFileContext = new FString(PathStr);

		// Use the buffer from the cached file data.
		*OutContents = CachedFileData->Buffer;

#if WITH_ICU_V78
		// ICU 78 requires the data length for integrity validation in initAliasData (ucnv_io.cpp).
		// OutLength is always non-null — ICU's uprv_mapFileFromCallback always passes a valid pointer.
		// An if-guard would silently return true without writing the length, causing udata_getLength()
		// to return -1 and a deferred U_INVALID_FORMAT_ERROR at u_init() rather than failing here.
		checkf(OutLength, TEXT("uprv_mapFileFromCallback must always pass a non-null OutLength — contract of UE Patch 2 to ICU 78 (umapfile.cpp)"));
		*OutLength = CachedFileData->FileSize;
#endif

		return true;
	}

	*OutFileContext = nullptr;
	*OutContents = nullptr;
	return false;
}

void FICUInternationalization::CloseDataFile(const void* InContext, void* const InFileContext, void* const InContents)
{
	FICUInternationalization* This = (FICUInternationalization*)InContext;
	check(This);

	// Early out on null context.
	if (!InFileContext)
	{
		return;
	}

	// The file context is the path to the file.
	FString* const Path = reinterpret_cast<FString*>(InFileContext);
	check(Path);

	// Look up the cached file data so we can maintain references.
	FScopeLock Lock(&This->PathToCachedFileDataMapCS);
	FICUCachedFileData* const CachedFileData = This->PathToCachedFileDataMap.Find(*Path);
	check(CachedFileData);
	check(CachedFileData->Buffer == InContents);

	// Remove a reference.
	--(CachedFileData->ReferenceCount);

	// If the last reference has been removed, the cached file data is not longer needed.
	if (CachedFileData->ReferenceCount == 0)
	{
		// Stat tracking.
#if STATS
		DataFileBytesInUseCount -= FMemory::GetAllocSize(CachedFileData->Buffer);
		if (FThreadStats::IsThreadingReady() && CachedDataFileBytesInUseCount != DataFileBytesInUseCount)
		{
			SET_MEMORY_STAT(STAT_MemoryICUDataFileAllocationSize, DataFileBytesInUseCount);
			CachedDataFileBytesInUseCount = DataFileBytesInUseCount;
		}
#endif

		This->TotalCachedFileDataBytes -= CachedFileData->FileSize;
		UE_LOGF(LogICUInternationalization, Verbose, "Unloaded: %ls (%.1f KB, %d files remain, %.1f KB total)",
			**Path, CachedFileData->FileSize / 1024.0, This->PathToCachedFileDataMap.Num() - 1, static_cast<double>(This->TotalCachedFileDataBytes) / 1024.0);

		// Delete the cached file data.
		This->PathToCachedFileDataMap.Remove(*Path);
	}

	// The path string we allocated for tracking is no longer necessary.
	delete Path;
}

FICUInternationalization::FICUCachedFileData::FICUCachedFileData(const int64 InFileSize)
	: ReferenceCount(0)
	, FileSize(static_cast<int32>(InFileSize))
	, Buffer(FICUOverrides::Malloc(nullptr, InFileSize))
{
	checkf(InFileSize <= static_cast<int64>(TNumericLimits<int32>::Max()), TEXT("ICU data file exceeds int32 capacity: %lld bytes"), InFileSize);
}

FICUInternationalization::FICUCachedFileData::FICUCachedFileData(FICUCachedFileData&& Source)
	: ReferenceCount(Source.ReferenceCount)
	, FileSize(Source.FileSize)
	, Buffer(Source.Buffer)
{
	Source.ReferenceCount = 0;
	Source.FileSize = 0;
	Source.Buffer = nullptr;
}

FICUInternationalization::FICUCachedFileData::~FICUCachedFileData()
{
	if (Buffer)
	{
		// Removing this check as the actual crash when the lingering ICU resource is 
		// deleted is much more useful at tracking down where the leak is coming from
		//check(ReferenceCount == 0);
		FICUOverrides::Free(nullptr, Buffer);
	}
}

#endif
