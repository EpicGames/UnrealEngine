// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Containers/Set.h"
#include "Containers/Map.h"
#include "Internationalization/CulturePointer.h"
#include "Templates/UniquePtr.h"
#include "Templates/PimplPtr.h"
#include "ICUUtilities.h"

class FCultureFilter;
class FInternationalization;

#if UE_ENABLE_ICU

THIRD_PARTY_INCLUDES_START
	#include <unicode/umachine.h>
PRAGMA_PUSH_PLATFORM_DEFAULT_PACKING
	#include <unicode/gregocal.h> // icu::Calendar can be affected by the non-standard packing UE4 uses, so force the platform default
PRAGMA_POP_PLATFORM_DEFAULT_PACKING
THIRD_PARTY_INCLUDES_END

// This should be defined by ICU.build.cs
#ifndef NEEDS_ICU_DLLS
	#define NEEDS_ICU_DLLS 0
#endif

class FICUInternationalization
{
public:
	FICUInternationalization(FInternationalization* const I18N);

	bool Initialize();
	void Terminate();

	void LoadAllCultureData();

	bool IsCultureRemapped(const FString& Name, FString* OutMappedCulture);
	bool IsCultureAllowed(const FString& Name);

	void RefreshCultureDisplayNames(const TArray<FString>& InPrioritizedDisplayCultureNames);
	void RefreshCachedConfigData();

	void HandleLanguageChanged(const FCultureRef InNewLanguage);
	void GetCultureNames(TArray<FString>& CultureNames) const;
	TArray<FString> GetPrioritizedCultureNames(const FString& Name);
	FCulturePtr GetCulture(const FString& Name);

	UDate UEDateTimeToICUDate(const FDateTime& DateTime);

private:
#if NEEDS_ICU_DLLS
	void LoadDLLs();
	void UnloadDLLs();
#endif

	void InitializeAvailableCultures();
	void ConditionalInitializeCultureMappings();
	void ConditionalInitializeAllowedCultures();

	enum class EAllowDefaultCultureFallback : uint8 { No, Yes, };
	FCulturePtr FindOrMakeCulture(const FString& Name, const EAllowDefaultCultureFallback AllowDefaultFallback);
	FCulturePtr FindOrMakeCanonizedCulture(const FString& Name, const EAllowDefaultCultureFallback AllowDefaultFallback);

	void InitializeTimeZone();
	void InitializeInvariantGregorianCalendar();

private:
	struct FICUCultureData
	{
		FString Name;
		FString LanguageCode;
		FString ScriptCode;
		FString CountryCode;

		bool operator==(const FICUCultureData& Other) const
		{
			return Name == Other.Name;
		}

		bool operator!=(const FICUCultureData& Other) const
		{
			return Name != Other.Name;
		}
	};

private:
	FInternationalization* const I18N;

	TArray<void*> DLLHandles;
	FString ICUDataDirectory;

	TArray<FICUCultureData> AllAvailableCultures;
	TMap<FString, int32> AllAvailableCulturesMap;
	TMap<FString, TArray<int32>> AllAvailableLanguagesToSubCulturesMap;

	bool bHasInitializedCultureMappings;
	TMap<FString, FString> CultureMappings;

	bool bHasInitializedAllowedCultures;
	TPimplPtr<FCultureFilter> AllowedCulturesFilter;

	TMap<FString, FCultureRef> CachedCultures;
	FCriticalSection CachedCulturesCS;

	TUniquePtr<icu::GregorianCalendar> InvariantGregorianCalendar;
	FCriticalSection InvariantGregorianCalendarCS;

	TArray<FString> CachedPrioritizedDisplayCultureNames;

#if WITH_ICU_V78
	static UBool OpenDataFile(const void* InContext, void** OutFileContext, void** OutContents, const char* InPath, int32_t* OutLength);
#else
	static UBool OpenDataFile(const void* InContext, void** OutFileContext, void** OutContents, const char* InPath);
#endif
	static void CloseDataFile(const void* InContext, void* const InFileContext, void* const InContents);

	// Tidy class for storing the count of references for an ICU data file and the file's data itself.
	struct FICUCachedFileData
	{
		FICUCachedFileData(const int64 FileSize);
		FICUCachedFileData(FICUCachedFileData&& Source);
		~FICUCachedFileData();

		FICUCachedFileData(const FICUCachedFileData&) = delete;
		FICUCachedFileData& operator=(const FICUCachedFileData&) = delete;

		uint32 ReferenceCount;
		// Stored as int32 because ICU's UDataMemory::length (udatamem.h) is int32_t — the value written
		// here is ultimately assigned to pData->length in uprv_mapFileFromCallback, so widening to int64
		// would not eliminate the narrowing; it would just move it into ICU's own internals. int32 also
		// avoids the 4-byte alignment padding that int64 would introduce after ReferenceCount, increasing
		// every cache entry by 8 bytes. No ICU data file approaches 2GB — the largest full locale set
		// is order of tens of MB.
		int32 FileSize;
		void* Buffer;
	};

	// Map for associating ICU data file paths with cached file data, to prevent multiple copies of immutable ICU data files from residing in memory.
	TMap<FString, FICUCachedFileData> PathToCachedFileDataMap;
	FCriticalSection PathToCachedFileDataMapCS;

	/**
	 * Running tally of memory across all cached ICU data files, in bytes.
	 * MUST only be read or written while holding PathToCachedFileDataMapCS.
	 * All access sites in OpenDataFile and CloseDataFile already hold that lock.
	 */
	int64 TotalCachedFileDataBytes = 0;
};

#endif
