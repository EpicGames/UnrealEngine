// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "Math/UnrealMathUtility.h"
#include "Containers/UnrealString.h"
#include "Templates/SharedPointer.h"
#include "Internationalization/IBreakIterator.h"
#include "Internationalization/BreakIterator.h"

#if UE_ENABLE_ICU
#include "Internationalization/ICUBreakIterator.h"
#include "Internationalization/ICUTextCharacterIterator.h"
#include "HAL/IConsoleManager.h"

enum class EHangulTextWrappingMethod : uint8
{
	/** Wrap per-syllable (default Geumchik wrapping) */
	PerSyllable = 0,

	/** Wrap per-word (preferred native wrapping) */
	PerWord,
};

static TAutoConsoleVariable<int32> CVarHangulTextWrappingMethod(
	TEXT("Localization.HangulTextWrappingMethod"),
	static_cast<int32>(EHangulTextWrappingMethod::PerWord),
	TEXT("0: PerSyllable, 1: PerWord (default)."),
	ECVF_Default
	);

EHangulTextWrappingMethod GetHangulTextWrappingMethod()
{
	const int32 HangulTextWrappingMethodAsInt = CVarHangulTextWrappingMethod.AsVariable()->GetInt();
	if (HangulTextWrappingMethodAsInt >= static_cast<int32>(EHangulTextWrappingMethod::PerSyllable) && HangulTextWrappingMethodAsInt <= static_cast<int32>(EHangulTextWrappingMethod::PerWord))
	{
		return static_cast<EHangulTextWrappingMethod>(HangulTextWrappingMethodAsInt);
	}
	return EHangulTextWrappingMethod::PerWord;
}

FORCEINLINE bool IsHangul(const UChar32 InChar)
{
	return InChar >= 0xAC00 && InChar <= 0xD7A3;
}

class FICULineBreakIterator : public IBreakIterator
{
public:
	FICULineBreakIterator();
	virtual ~FICULineBreakIterator();

	virtual void SetString(FString&& InString) override;
	virtual void SetStringRef(FStringView InString) override;

	virtual int32 GetCurrentPosition() const override;

	virtual int32 ResetToBeginning() override;
	virtual int32 ResetToEnd() override;

	virtual int32 MoveToPrevious() override;
	virtual int32 MoveToNext() override;
	virtual int32 MoveToCandidateBefore(const int32 InIndex) override;
	virtual int32 MoveToCandidateAfter(const int32 InIndex) override;

protected:
	int32 MoveToPreviousImpl();
	int32 MoveToNextImpl();

	TSharedRef<icu::BreakIterator> GetInternalLineBreakIterator() const;

private:
	TWeakPtr<icu::BreakIterator> ICULineBreakIteratorHandle;
	int32 CurrentPosition;
};

FICULineBreakIterator::FICULineBreakIterator()
	: ICULineBreakIteratorHandle(FICUBreakIteratorManager::Get().CreateLineBreakIterator())
	, CurrentPosition(0)
{
}

FICULineBreakIterator::~FICULineBreakIterator()
{
	if (FICUBreakIteratorManager::IsInitialized())
	{
		// This assumes that FICULineBreakIterator owns the iterators, and that nothing ever copies an FICULineBreakIterator instance
		FICUBreakIteratorManager::Get().DestroyIterator(ICULineBreakIteratorHandle);
	}
}

void FICULineBreakIterator::SetString(FString&& InString)
{
	GetInternalLineBreakIterator()->adoptText(new FICUTextCharacterIterator(MoveTemp(InString))); // ICUBreakIterator takes ownership of this instance
	ResetToBeginning();
}

void FICULineBreakIterator::SetStringRef(FStringView InString)
{
	GetInternalLineBreakIterator()->adoptText(new FICUTextCharacterIterator(InString)); // ICUBreakIterator takes ownership of this instance
	ResetToBeginning();
}

int32 FICULineBreakIterator::GetCurrentPosition() const
{
	return CurrentPosition;
}

int32 FICULineBreakIterator::ResetToBeginning()
{
	TSharedRef<icu::BreakIterator> LineBrkIt = GetInternalLineBreakIterator();
	CurrentPosition = LineBrkIt->first();
	CurrentPosition = static_cast<FICUTextCharacterIterator&>(LineBrkIt->getText()).InternalIndexToSourceIndex(CurrentPosition);
	return CurrentPosition;
}

int32 FICULineBreakIterator::ResetToEnd()
{
	TSharedRef<icu::BreakIterator> LineBrkIt = GetInternalLineBreakIterator();
	CurrentPosition = LineBrkIt->last();
	CurrentPosition = static_cast<FICUTextCharacterIterator&>(LineBrkIt->getText()).InternalIndexToSourceIndex(CurrentPosition);
	return CurrentPosition;
}

int32 FICULineBreakIterator::MoveToPrevious()
{
	return MoveToPreviousImpl();
}

int32 FICULineBreakIterator::MoveToNext()
{
	return MoveToNextImpl();
}

int32 FICULineBreakIterator::MoveToCandidateBefore(const int32 InIndex)
{
	CurrentPosition = InIndex;
	return MoveToPreviousImpl();
}

int32 FICULineBreakIterator::MoveToCandidateAfter(const int32 InIndex)
{
	CurrentPosition = InIndex;
	return MoveToNextImpl();
}

int32 FICULineBreakIterator::MoveToPreviousImpl()
{
	TSharedRef<icu::BreakIterator> LineBrkIt = GetInternalLineBreakIterator();
	FICUTextCharacterIterator& CharIt = static_cast<FICUTextCharacterIterator&>(LineBrkIt->getText());

	int32 InternalPosition = CharIt.SourceIndexToInternalIndex(CurrentPosition);

	// For Hangul using per-word wrapping, we walk back to the first Hangul character in the word and use that as the starting point for the 
	// line-break iterator, as this will correctly handle the remaining Geumchik wrapping rules, without also causing per-syllable wrapping
	if (GetHangulTextWrappingMethod() == EHangulTextWrappingMethod::PerWord)
	{
		CharIt.setIndex32(InternalPosition);

		if (IsHangul(CharIt.current32()))
		{
			// Walk to the start of the Hangul characters
			while (CharIt.hasPrevious() && IsHangul(CharIt.previous32()))
			{
				InternalPosition = CharIt.getIndex();
			}
		}
	}

	InternalPosition = LineBrkIt->preceding(InternalPosition);
	CurrentPosition = CharIt.InternalIndexToSourceIndex(InternalPosition);

	return CurrentPosition;
}

int32 FICULineBreakIterator::MoveToNextImpl()
{
	TSharedRef<icu::BreakIterator> LineBrkIt = GetInternalLineBreakIterator();
	FICUTextCharacterIterator& CharIt = static_cast<FICUTextCharacterIterator&>(LineBrkIt->getText());

	int32 InternalPosition = CharIt.SourceIndexToInternalIndex(CurrentPosition);

	// For Hangul using per-word wrapping, we walk forward to the last Hangul character in the word and use that as the starting point for the 
	// line-break iterator, as this will correctly handle the remaining Geumchik wrapping rules, without also causing per-syllable wrapping
	if (GetHangulTextWrappingMethod() == EHangulTextWrappingMethod::PerWord)
	{
		CharIt.setIndex32(InternalPosition);

		if (IsHangul(CharIt.current32()))
		{
			// Walk to the end of the Hangul characters
			while (CharIt.hasNext() && IsHangul(CharIt.next32()))
			{
				InternalPosition = CharIt.getIndex();
			}
		}
	}

	InternalPosition = LineBrkIt->following(InternalPosition);
	CurrentPosition = CharIt.InternalIndexToSourceIndex(InternalPosition);

	return CurrentPosition;
}

TSharedRef<icu::BreakIterator> FICULineBreakIterator::GetInternalLineBreakIterator() const
{
	return ICULineBreakIteratorHandle.Pin().ToSharedRef();
}

TSharedRef<IBreakIterator> FBreakIterator::CreateLineBreakIterator()
{
	return MakeShareable(new FICULineBreakIterator());
}


TSharedRef<IBreakIterator> FBreakIterator::GetLineBreakIterator()
{
	static thread_local TSharedPtr<IBreakIterator> CachedLineBreakIterator;
	if (!CachedLineBreakIterator.IsValid())
	{
		CachedLineBreakIterator = MakeShareable(new FICULineBreakIterator());
	}

	check(CachedLineBreakIterator.IsValid());
	return CachedLineBreakIterator.ToSharedRef();
}

#endif
