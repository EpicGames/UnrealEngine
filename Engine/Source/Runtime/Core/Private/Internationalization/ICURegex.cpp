// Copyright Epic Games, Inc. All Rights Reserved.

#include "Internationalization/ICURegex.h"
#include "Misc/ScopeLock.h"

#if UE_ENABLE_ICU
#include "Internationalization/Regex.h"
#include "Internationalization/ICUUtilities.h"

class FRegexPatternImplementation
{
public:
	static uint32_t GetICURegexFlags(ERegexPatternFlags Flags)
	{
		uint32_t OutFlags = 0;
		if (EnumHasAnyFlags(Flags, ERegexPatternFlags::CaseInsensitive))
		{
			OutFlags |= UREGEX_CASE_INSENSITIVE;
		}

		return OutFlags;
	}

	FRegexPatternImplementation(const FString& SourceString, ERegexPatternFlags Flags)
		: ICURegexPattern(FICURegexManager::Get().CreateRegexPattern(SourceString, GetICURegexFlags(Flags)))
	{
	}

	~FRegexPatternImplementation()
	{
		if (FICURegexManager::IsInitialized())
		{
			FICURegexManager::Get().DestroyRegexPattern(ICURegexPattern);
		}
	}

	TSharedPtr<const icu::RegexPattern> GetInternalRegexPattern() const
	{
		return ICURegexPattern.Pin();
	}

private:
	TWeakPtr<const icu::RegexPattern> ICURegexPattern;
};


class FRegexMatcherImplementation
{
public:
	FRegexMatcherImplementation(const FRegexPatternImplementation& Pattern, const FString& InputString)
		: ICUString(ICUUtilities::ConvertString(InputString))
		, ICURegexMatcher(FICURegexManager::Get().CreateRegexMatcher(Pattern.GetInternalRegexPattern().Get(), &ICUString))
		, OriginalString(InputString)
	{
	}

	~FRegexMatcherImplementation()
	{
		if (FICURegexManager::IsInitialized())
		{
			FICURegexManager::Get().DestroyRegexMatcher(ICURegexMatcher);
		}
	}

	TSharedPtr<icu::RegexMatcher> GetInternalRegexMatcher() const
	{
		return ICURegexMatcher.Pin();
	}

	const FString& GetInternalString() const
	{
		return OriginalString;
	}

private:
	const icu::UnicodeString ICUString; // ICURegexMatcher keeps a reference to this string internally
	TWeakPtr<icu::RegexMatcher> ICURegexMatcher;
	FString OriginalString;
};


FICURegexManager* FICURegexManager::Singleton = nullptr;

void FICURegexManager::Create()
{
	check(!Singleton);
	Singleton = new FICURegexManager();
}

void FICURegexManager::Destroy()
{
	check(Singleton);
	delete Singleton;
	Singleton = nullptr;
}

bool FICURegexManager::IsInitialized()
{
	return Singleton != nullptr;
}

FICURegexManager& FICURegexManager::Get()
{
	check(Singleton);
	return *Singleton;
}

TWeakPtr<const icu::RegexPattern> FICURegexManager::CreateRegexPattern(const FString& InSourceString, uint32_t InICURegexFlags)
{
	icu::UnicodeString ICUSourceString;
	ICUUtilities::ConvertString(InSourceString, ICUSourceString);

	UErrorCode ICUStatus = U_ZERO_ERROR;
	TSharedPtr<const icu::RegexPattern> ICURegexPattern = MakeShareable(icu::RegexPattern::compile(ICUSourceString, InICURegexFlags, ICUStatus));

	if (ICURegexPattern.IsValid())
	{
		FScopeLock ScopeLock(&AllocatedRegexPatternsCS);
		AllocatedRegexPatterns.Add(ICURegexPattern);
	}

	return ICURegexPattern;
}

void FICURegexManager::DestroyRegexPattern(TWeakPtr<const icu::RegexPattern>& InICURegexPattern)
{
	TSharedPtr<const icu::RegexPattern> ICURegexPattern = InICURegexPattern.Pin();
	if (ICURegexPattern.IsValid())
	{
		FScopeLock ScopeLock(&AllocatedRegexPatternsCS);
		AllocatedRegexPatterns.Remove(ICURegexPattern);
	}
	InICURegexPattern.Reset();
}

TWeakPtr<icu::RegexMatcher> FICURegexManager::CreateRegexMatcher(const icu::RegexPattern* InPattern, const icu::UnicodeString* InInputString)
{
	TSharedPtr<icu::RegexMatcher> ICURegexMatcher;

	if (InPattern && InInputString)
	{
		UErrorCode ICUStatus = U_ZERO_ERROR;
		ICURegexMatcher = MakeShareable(InPattern->matcher(*InInputString, ICUStatus));
	}

	if (ICURegexMatcher.IsValid())
	{
		FScopeLock ScopeLock(&AllocatedRegexMatchersCS);
		AllocatedRegexMatchers.Add(ICURegexMatcher);
	}

	return ICURegexMatcher;
}

void FICURegexManager::DestroyRegexMatcher(TWeakPtr<icu::RegexMatcher>& InICURegexMatcher)
{
	TSharedPtr<icu::RegexMatcher> ICURegexMatcher = InICURegexMatcher.Pin();
	if (ICURegexMatcher.IsValid())
	{
		FScopeLock ScopeLock(&AllocatedRegexMatchersCS);
		AllocatedRegexMatchers.Remove(ICURegexMatcher);
	}
	InICURegexMatcher.Reset();
}


FRegexPattern::FRegexPattern(const FString& SourceString, ERegexPatternFlags Flags /*= ERegexPatternFlags::None*/) 
	: Implementation(MakeShared<FRegexPatternImplementation>(SourceString, Flags))
{
}


FRegexMatcher::FRegexMatcher(const FRegexPattern& SourcePattern, const FString& InputString)
	: Pattern(SourcePattern)
	, Implementation(MakeShared<FRegexMatcherImplementation>(Pattern.Implementation.Get(), InputString))
{
}

FRegexMatcher::FRegexMatcher(FRegexPattern&& SourcePattern, const FString& InputString)
	: Pattern(MoveTemp(SourcePattern))
	, Implementation(MakeShared<FRegexMatcherImplementation>(Pattern.Implementation.Get(), InputString))
{
}

bool FRegexMatcher::FindNext()
{
	TSharedPtr<icu::RegexMatcher> ICURegexMatcher = Implementation->GetInternalRegexMatcher();
	return ICURegexMatcher.IsValid() && ICURegexMatcher->find() != 0;
}

FString FRegexMatcher::ReplaceFirst(FStringView Replacement)
{
	TSharedPtr<icu::RegexMatcher> ICURegexMatcher = Implementation->GetInternalRegexMatcher();
	if (ICURegexMatcher.IsValid())
	{
		UErrorCode ICUStatus = U_ZERO_ERROR;
		icu::UnicodeString ReplacementString = ICUUtilities::ConvertString(Replacement);
		icu::UnicodeString Result = ICURegexMatcher->replaceFirst(ReplacementString, ICUStatus);
		return ICUUtilities::ConvertString(Result);
	}

	return FString(Replacement);
}

FString FRegexMatcher::ReplaceAll(FStringView Replacement)
{
	TSharedPtr<icu::RegexMatcher> ICURegexMatcher = Implementation->GetInternalRegexMatcher();
	if (ICURegexMatcher.IsValid())
	{
		UErrorCode ICUStatus = U_ZERO_ERROR;
		icu::UnicodeString ReplacementString = ICUUtilities::ConvertString(Replacement);
		icu::UnicodeString Result = ICURegexMatcher->replaceAll(ReplacementString, ICUStatus);
		return ICUUtilities::ConvertString(Result);
	}

	return FString(Replacement);
}

int32 FRegexMatcher::GetMatchBeginning()
{
	UErrorCode ICUStatus = U_ZERO_ERROR;
	TSharedPtr<icu::RegexMatcher> ICURegexMatcher = Implementation->GetInternalRegexMatcher();
	return ICURegexMatcher.IsValid() ? ICURegexMatcher->start(ICUStatus) : INDEX_NONE;
}

int32 FRegexMatcher::GetMatchEnding()
{
	UErrorCode ICUStatus = U_ZERO_ERROR;
	TSharedPtr<icu::RegexMatcher> ICURegexMatcher = Implementation->GetInternalRegexMatcher();
	return ICURegexMatcher.IsValid() ? ICURegexMatcher->end(ICUStatus) : INDEX_NONE;
}

int32 FRegexMatcher::GetCaptureGroupBeginning(const int32 Index)
{
	UErrorCode ICUStatus = U_ZERO_ERROR;
	TSharedPtr<icu::RegexMatcher> ICURegexMatcher = Implementation->GetInternalRegexMatcher();
	return ICURegexMatcher.IsValid() ? ICURegexMatcher->start(Index, ICUStatus) : INDEX_NONE;
}

int32 FRegexMatcher::GetCaptureGroupEnding(const int32 Index)
{
	UErrorCode ICUStatus = U_ZERO_ERROR;
	TSharedPtr<icu::RegexMatcher> ICURegexMatcher = Implementation->GetInternalRegexMatcher();
	return ICURegexMatcher.IsValid() ? ICURegexMatcher->end(Index, ICUStatus) : INDEX_NONE;
}

FString FRegexMatcher::GetCaptureGroup(const int32 Index)
{
	int32 CaptureGroupBeginning = 0;
	int32 CaptureGroupEnding = 0;

	TSharedPtr<icu::RegexMatcher> ICURegexMatcher = Implementation->GetInternalRegexMatcher();
	if (ICURegexMatcher.IsValid())
	{
		UErrorCode ICUStatus = U_ZERO_ERROR;
		CaptureGroupBeginning = ICURegexMatcher->start(Index, ICUStatus);
		CaptureGroupEnding = ICURegexMatcher->end(Index, ICUStatus);
	}

	CaptureGroupBeginning = FMath::Max(0, CaptureGroupBeginning);
	CaptureGroupEnding = FMath::Max(CaptureGroupBeginning, CaptureGroupEnding);

	return Implementation->GetInternalString().Mid(CaptureGroupBeginning, CaptureGroupEnding - CaptureGroupBeginning);
}

int32 FRegexMatcher::GetBeginLimit()
{
	TSharedPtr<icu::RegexMatcher> ICURegexMatcher = Implementation->GetInternalRegexMatcher();
	return ICURegexMatcher.IsValid() ? ICURegexMatcher->regionStart() : INDEX_NONE;
}

int32 FRegexMatcher::GetEndLimit()
{
	TSharedPtr<icu::RegexMatcher> ICURegexMatcher = Implementation->GetInternalRegexMatcher();
	return ICURegexMatcher.IsValid() ? ICURegexMatcher->regionEnd() : INDEX_NONE;
}

void FRegexMatcher::SetLimits(const int32 BeginIndex, const int32 EndIndex)
{
	TSharedPtr<icu::RegexMatcher> ICURegexMatcher = Implementation->GetInternalRegexMatcher();
	if (ICURegexMatcher.IsValid())
	{
		UErrorCode ICUStatus = U_ZERO_ERROR;
		ICURegexMatcher->region(BeginIndex, EndIndex, ICUStatus);
	}
}

#endif
