// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "Templates/SharedPointer.h"

#if UE_ENABLE_ICU
THIRD_PARTY_INCLUDES_START
	#include <unicode/regex.h>
THIRD_PARTY_INCLUDES_END

/**
 * Manages the lifespan of ICU regex objects
 */
class FICURegexManager
{
public:
	static void Create();
	static void Destroy();
	static bool IsInitialized();
	static FICURegexManager& Get();

	TWeakPtr<const icu::RegexPattern> CreateRegexPattern(const FString& InSourceString, uint32_t InICURegexFlags);
	void DestroyRegexPattern(TWeakPtr<const icu::RegexPattern>& InICURegexPattern);

	TWeakPtr<icu::RegexMatcher> CreateRegexMatcher(const icu::RegexPattern* InPattern, const icu::UnicodeString* InInputString);
	void DestroyRegexMatcher(TWeakPtr<icu::RegexMatcher>& InICURegexMatcher);

private:
	static FICURegexManager* Singleton;

	FCriticalSection AllocatedRegexPatternsCS;
	TSet<TSharedPtr<const icu::RegexPattern>> AllocatedRegexPatterns;

	FCriticalSection AllocatedRegexMatchersCS;
	TSet<TSharedPtr<icu::RegexMatcher>> AllocatedRegexMatchers;
};

#endif
