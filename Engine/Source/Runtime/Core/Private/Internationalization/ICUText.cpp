// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "Containers/UnrealString.h"
#include "Logging/LogMacros.h"
#include "Templates/SharedPointer.h"
#include "Internationalization/Text.h"
#include "Internationalization/TextChar.h"
#include "Internationalization/TextChronoFormatter.h"
#include "Internationalization/TextTransformer.h"
#include "Internationalization/Internationalization.h"

#if UE_ENABLE_ICU
#include "Internationalization/TextHistory.h"
#include "HAL/IConsoleManager.h"

THIRD_PARTY_INCLUDES_START
	#include <unicode/utypes.h>
	#include <unicode/unistr.h>
	#include <unicode/coll.h>
	#include <unicode/sortkey.h>
	#include <unicode/ubidi.h>
THIRD_PARTY_INCLUDES_END

#include "Internationalization/ICUUtilities.h"
#include "Internationalization/ICUCulture.h"
#include "Internationalization/ICUInternationalization.h"
#include "Internationalization/ICUTextCharacterIterator.h"

FString FTextChronoFormatter::AsDate(const FDateTime& DateTime, const EDateTimeStyle::Type DateStyle, const FString& TimeZone, const FCulture& TargetCulture)
{
	FInternationalization& I18N = FInternationalization::Get();
	checkf(I18N.IsInitialized() == true, TEXT("FInternationalization is not initialized. An FText formatting method was likely used in static object initialization - this is not supported."));
	const UDate ICUDate = I18N.Implementation->UEDateTimeToICUDate(DateTime);

	const TSharedRef<const icu::DateFormat, ESPMode::ThreadSafe> ICUDateFormat( TargetCulture.Implementation->GetDateFormatter(DateStyle, TimeZone) );
	icu::UnicodeString FormattedString;
	ICUDateFormat->format(ICUDate, FormattedString);

	return ICUUtilities::ConvertString(FormattedString);
}

FString FTextChronoFormatter::AsTime(const FDateTime& DateTime, const EDateTimeStyle::Type TimeStyle, const FString& TimeZone, const FCulture& TargetCulture)
{
	FInternationalization& I18N = FInternationalization::Get();
	checkf(I18N.IsInitialized() == true, TEXT("FInternationalization is not initialized. An FText formatting method was likely used in static object initialization - this is not supported."));
	const UDate ICUDate = I18N.Implementation->UEDateTimeToICUDate(DateTime);

	const TSharedRef<const icu::DateFormat, ESPMode::ThreadSafe> ICUDateFormat( TargetCulture.Implementation->GetTimeFormatter(TimeStyle, TimeZone) );
	icu::UnicodeString FormattedString;
	ICUDateFormat->format(ICUDate, FormattedString);

	return ICUUtilities::ConvertString(FormattedString);
}

FString FTextChronoFormatter::AsDateTime(const FDateTime& DateTime, const EDateTimeStyle::Type DateStyle, const EDateTimeStyle::Type TimeStyle, const FString& TimeZone, const FCulture& TargetCulture)
{
	FInternationalization& I18N = FInternationalization::Get();
	checkf(I18N.IsInitialized() == true, TEXT("FInternationalization is not initialized. An FText formatting method was likely used in static object initialization - this is not supported."));
	const UDate ICUDate = I18N.Implementation->UEDateTimeToICUDate(DateTime);

	const TSharedRef<const icu::DateFormat, ESPMode::ThreadSafe> ICUDateFormat( TargetCulture.Implementation->GetDateTimeFormatter(DateStyle, TimeStyle, TimeZone) );
	icu::UnicodeString FormattedString;
	ICUDateFormat->format(ICUDate, FormattedString);

	return ICUUtilities::ConvertString(FormattedString);
}

FString FTextChronoFormatter::AsDateTime(const FDateTime& DateTime, const FString& CustomPattern, const FString& TimeZone, const FCulture& TargetCulture)
{
	FInternationalization& I18N = FInternationalization::Get();
	checkf(I18N.IsInitialized() == true, TEXT("FInternationalization is not initialized. An FText formatting method was likely used in static object initialization - this is not supported."));
	const UDate ICUDate = I18N.Implementation->UEDateTimeToICUDate(DateTime);

	const TSharedRef<const icu::DateFormat, ESPMode::ThreadSafe> ICUDateFormat(TargetCulture.Implementation->GetDateTimeFormatter(CustomPattern, TimeZone));
	icu::UnicodeString FormattedString;
	ICUDateFormat->format(ICUDate, FormattedString);

	return ICUUtilities::ConvertString(FormattedString);
}

FString FTextTransformer::ToLower(const FString& InStr)
{
	FInternationalization& I18N = FInternationalization::Get();
	checkf(I18N.IsInitialized() == true, TEXT("FInternationalization is not initialized. An FText formatting method was likely used in static object initialization - this is not supported."));
	
	// We run this code in the open because it calls into ICU which is not transactionally instrumented.
	// We get away with this because we use the explicit `AutoRTFM::Open` which knows how to take open
	// constructed `FString`'s and make them transactionally safe (see `AutoRTFMReturnFromOpenMode`).
	return AutoRTFM::Open([&]
	{
		return ICUUtilities::ConvertString(ICUUtilities::ConvertString(InStr).toLower());
	});
}

FString FTextTransformer::ToUpper(const FString& InStr)
{
	FInternationalization& I18N = FInternationalization::Get();
	checkf(I18N.IsInitialized() == true, TEXT("FInternationalization is not initialized. An FText formatting method was likely used in static object initialization - this is not supported."));

	// We run this code in the open because it calls into ICU which is not transactionally instrumented.
	// We get away with this because we use the explicit `AutoRTFM::Open` which knows how to take open
	// constructed `FString`'s and make them transactionally safe (see `AutoRTFMReturnFromOpenMode`).
	return AutoRTFM::Open([&]
	{
		return ICUUtilities::ConvertString(ICUUtilities::ConvertString(InStr).toUpper());
	});
}

bool FTextChar::IsAlpha(const UTF32CHAR Codepoint)
{
	return u_isalpha(Codepoint) != 0;
}

bool FTextChar::IsGraph(const UTF32CHAR Codepoint)
{
	return u_isgraph(Codepoint) != 0;
}

bool FTextChar::IsPrint(const UTF32CHAR Codepoint)
{
	return u_isprint(Codepoint) != 0;
}

bool FTextChar::IsPunct(const UTF32CHAR Codepoint)
{
	return u_ispunct(Codepoint) != 0;
}

bool FTextChar::IsAlnum(const UTF32CHAR Codepoint)
{
	return u_isalnum(Codepoint) != 0;
}

bool FTextChar::IsDigit(const UTF32CHAR Codepoint)
{
	return u_isdigit(Codepoint) != 0;
}

bool FTextChar::IsHexDigit(const UTF32CHAR Codepoint)
{
	return u_isxdigit(Codepoint) != 0;
}

bool FTextChar::IsControl(const UTF32CHAR Codepoint)
{
	return u_iscntrl(Codepoint) != 0;
}

bool FTextChar::IsWhitespace(const UTF32CHAR Codepoint)
{
	return u_isWhitespace(Codepoint) != 0;
}

bool FTextChar::IsUpper(const UTF32CHAR Codepoint)
{
	return u_isupper(Codepoint) != 0;
}

bool FTextChar::IsLower(const UTF32CHAR Codepoint)
{
	return u_islower(Codepoint) != 0;
}

UTF32CHAR FTextChar::ToLower(const UTF32CHAR Codepoint)
{
	return u_tolower(Codepoint);
}

UTF32CHAR FTextChar::ToUpper(const UTF32CHAR Codepoint)
{
	return u_toupper(Codepoint);
}

int32 FTextComparison::CompareTo( const FString& A, const FString& B, const ETextComparisonLevel::Type ComparisonLevel )
{
	const TSharedRef<const icu::Collator, ESPMode::ThreadSafe> Collator( FInternationalization::Get().GetCurrentLanguage()->Implementation->GetCollator(ComparisonLevel) );

	// Create an iterator for 'A' so that we can interface with ICU
	FStringView AView = A;
	UCharIterator ADisplayStringICUIterator;
	FICUTextCharacterIterator ADisplayStringIterator(AView);
	uiter_setCharacterIterator(&ADisplayStringICUIterator, &ADisplayStringIterator);

	// Create an iterator for 'B' so that we can interface with ICU
	FStringView BView = B;
	UCharIterator BDisplayStringICUIterator;
	FICUTextCharacterIterator BDisplayStringIterator(BView);
	uiter_setCharacterIterator(&BDisplayStringICUIterator, &BDisplayStringIterator);

	UErrorCode ICUStatus = U_ZERO_ERROR;
	const UCollationResult Result = Collator->compare(ADisplayStringICUIterator, BDisplayStringICUIterator, ICUStatus);

	return Result;
}

int32 FTextComparison::CompareToCaseIgnored( const FString& A, const FString& B )
{
	return CompareTo(A, B, ETextComparisonLevel::Secondary);
}

bool FTextComparison::EqualTo( const FString& A, const FString& B, const ETextComparisonLevel::Type ComparisonLevel )
{
	return CompareTo(A, B, ComparisonLevel) == 0;
}

bool FTextComparison::EqualToCaseIgnored( const FString& A, const FString& B )
{
	return EqualTo(A, B, ETextComparisonLevel::Secondary);
}

class FText::FSortPredicate::FSortPredicateImplementation
{
public:
	FSortPredicateImplementation(const ETextComparisonLevel::Type InComparisonLevel)
		: ICUCollator(FInternationalization::Get().GetCurrentLanguage()->Implementation->GetCollator(InComparisonLevel))
	{
	}

	bool Compare(const FText& A, const FText& B)
	{
		// Create an iterator for 'A' so that we can interface with ICU
		UCharIterator ADisplayStringICUIterator;
		FICUTextCharacterIterator ADisplayStringIterator(FStringView(A.ToString()));
		uiter_setCharacterIterator(&ADisplayStringICUIterator, &ADisplayStringIterator);

		// Create an iterator for 'B' so that we can interface with ICU
		UCharIterator BDisplayStringICUIterator;
		FICUTextCharacterIterator BDisplayStringIterator(FStringView(B.ToString()));
		uiter_setCharacterIterator(&BDisplayStringICUIterator, &BDisplayStringIterator);

		UErrorCode ICUStatus = U_ZERO_ERROR;
		const UCollationResult Result = ICUCollator->compare(ADisplayStringICUIterator, BDisplayStringICUIterator, ICUStatus);

		return Result != UCOL_GREATER;
	}

private:
	const TSharedRef<const icu::Collator, ESPMode::ThreadSafe> ICUCollator;
};

FText::FSortPredicate::FSortPredicate(const ETextComparisonLevel::Type ComparisonLevel)
	: Implementation( new FSortPredicateImplementation(ComparisonLevel) )
{

}

bool FText::FSortPredicate::operator()(const FText& A, const FText& B) const
{
	return Implementation->Compare(A, B);
}

namespace TextBiDi
{

namespace Internal
{

FORCEINLINE ETextDirection ICUToUE(const UBiDiDirection InDirection)
{
	switch (InDirection)
	{
	case UBIDI_LTR:
		return ETextDirection::LeftToRight;
	case UBIDI_RTL:
		return ETextDirection::RightToLeft;
	case UBIDI_MIXED:
		return ETextDirection::Mixed;
	default:
		break;
	}

	return ETextDirection::LeftToRight;
}

UBiDiLevel GetParagraphDirection(const ETextDirection InBaseDirection)
{
	check(InBaseDirection != ETextDirection::Mixed);
	return (InBaseDirection == ETextDirection::LeftToRight) ? 0 : 1; // 0 = LTR, 1 = RTL
}

ETextDirection ComputeTextDirection(UBiDi* InICUBiDi, const icu::UnicodeString& InICUString)
{
	UErrorCode ICUStatus = U_ZERO_ERROR;

	ubidi_setPara(InICUBiDi, InICUString.getBuffer(), InICUString.length(), GetParagraphDirection(ETextDirection::LeftToRight), nullptr, &ICUStatus);

	if (U_SUCCESS(ICUStatus))
	{
		return Internal::ICUToUE(ubidi_getDirection(InICUBiDi));
	}
	else
	{
		UE_LOGF(LogCore, Warning, "Failed to set the string data on the ICU BiDi object (error code: %d). Text will assumed to be left-to-right", static_cast<int32>(ICUStatus));
	}

	return ETextDirection::LeftToRight;
}

ETextDirection ComputeTextDirection(UBiDi* InICUBiDi, const icu::UnicodeString& InICUString, const int32 InStringOffset, const ETextDirection InBaseDirection, TArray<FTextDirectionInfo>& OutTextDirectionInfo)
{
	UErrorCode ICUStatus = U_ZERO_ERROR;

	ubidi_setPara(InICUBiDi, InICUString.getBuffer(), InICUString.length(), GetParagraphDirection(InBaseDirection), nullptr, &ICUStatus);

	if (U_SUCCESS(ICUStatus))
	{
		const ETextDirection ReturnDirection = Internal::ICUToUE(ubidi_getDirection(InICUBiDi));

		const int32 RunCount = ubidi_countRuns(InICUBiDi, &ICUStatus);
		OutTextDirectionInfo.AddZeroed(RunCount);
		for (int32 RunIndex = 0; RunIndex < RunCount; ++RunIndex)
		{
			FTextDirectionInfo& CurTextDirectionInfo = OutTextDirectionInfo[RunIndex];

			int32 InternalStartIndex = 0;
			int32 InternalLength = 0;
			CurTextDirectionInfo.TextDirection = Internal::ICUToUE(ubidi_getVisualRun(InICUBiDi, RunIndex, &InternalStartIndex, &InternalLength));
			
			// Adjust the index and length for what FString expects (ICU always uses UTF-16 indices internally, and FString might not be UTF-16)
			CurTextDirectionInfo.StartIndex = InStringOffset + ICUUtilities::GetNativeStringLength(InICUString, 0, InternalStartIndex);
			CurTextDirectionInfo.Length = ICUUtilities::GetNativeStringLength(InICUString, InternalStartIndex, InternalLength);
		}

		return ReturnDirection;
	}
	else
	{
		UE_LOGF(LogCore, Warning, "Failed to set the string data on the ICU BiDi object (error code: %d). Text will assumed to be left-to-right", static_cast<int32>(ICUStatus));
	}

	return ETextDirection::LeftToRight;
}

ETextDirection ComputeBaseDirection(const icu::UnicodeString& InICUString)
{
	const UBiDiDirection ICUBaseDirection = ubidi_getBaseDirection(InICUString.getBuffer(), InICUString.length());

	// ICUToUE will treat UBIDI_NEUTRAL as LTR
	return Internal::ICUToUE(ICUBaseDirection);
}

class FICUTextBiDi : public ITextBiDi
{
public:
	FICUTextBiDi()
		: ICUBiDi(ubidi_open())
	{
	}

	~FICUTextBiDi()
	{
		ubidi_close(ICUBiDi);
		ICUBiDi = nullptr;
	}

	virtual ETextDirection ComputeTextDirection(const FText& InText) override
	{
		return FICUTextBiDi::ComputeTextDirection(InText.ToString());
	}

	virtual ETextDirection ComputeTextDirection(const FString& InString) override
	{
		return FICUTextBiDi::ComputeTextDirection(*InString, 0, InString.Len());
	}

	virtual ETextDirection ComputeTextDirection(const TCHAR* InString, const int32 InStringStartIndex, const int32 InStringLen) override
	{
		if (InStringLen == 0)
		{
			return ETextDirection::LeftToRight;
		}

		StringConverter.ConvertString(InString, InStringStartIndex, InStringLen, ICUString);

		return Internal::ComputeTextDirection(ICUBiDi, ICUString);
	}

	virtual ETextDirection ComputeTextDirection(const FText& InText, const ETextDirection InBaseDirection, TArray<FTextDirectionInfo>& OutTextDirectionInfo) override
	{
		return FICUTextBiDi::ComputeTextDirection(InText.ToString(), InBaseDirection, OutTextDirectionInfo);
	}

	virtual ETextDirection ComputeTextDirection(const FString& InString, const ETextDirection InBaseDirection, TArray<FTextDirectionInfo>& OutTextDirectionInfo) override
	{
		return FICUTextBiDi::ComputeTextDirection(*InString, 0, InString.Len(), InBaseDirection, OutTextDirectionInfo);
	}

	virtual ETextDirection ComputeTextDirection(const TCHAR* InString, const int32 InStringStartIndex, const int32 InStringLen, const ETextDirection InBaseDirection, TArray<FTextDirectionInfo>& OutTextDirectionInfo) override
	{
		OutTextDirectionInfo.Reset();

		if (InStringLen == 0)
		{
			return ETextDirection::LeftToRight;
		}

		StringConverter.ConvertString(InString, InStringStartIndex, InStringLen, ICUString);

		return Internal::ComputeTextDirection(ICUBiDi, ICUString, InStringStartIndex, InBaseDirection, OutTextDirectionInfo);
	}

	virtual ETextDirection ComputeBaseDirection(const FText& InText) override
	{
		return FICUTextBiDi::ComputeBaseDirection(InText.ToString());
	}

	virtual ETextDirection ComputeBaseDirection(const FString& InString) override
	{
		return FICUTextBiDi::ComputeBaseDirection(*InString, 0, InString.Len());
	}

	virtual ETextDirection ComputeBaseDirection(const TCHAR* InString, const int32 InStringStartIndex, const int32 InStringLen) override
	{
		if (InStringLen == 0)
		{
			return ETextDirection::LeftToRight;
		}

		StringConverter.ConvertString(InString, InStringStartIndex, InStringLen, ICUString);

		return Internal::ComputeBaseDirection(ICUString);
	}

private:
	/** Non-copyable */
	FICUTextBiDi(const FICUTextBiDi&);
	FICUTextBiDi& operator=(const FICUTextBiDi&);

	UBiDi* ICUBiDi;
	icu::UnicodeString ICUString;
	ICUUtilities::FStringConverter StringConverter;
};

} // namespace Internal

TUniquePtr<ITextBiDi> CreateTextBiDi()
{
	return MakeUnique<Internal::FICUTextBiDi>();
}

ETextDirection ComputeTextDirection(const FText& InText)
{
	return ComputeTextDirection(InText.ToString());
}

ETextDirection ComputeTextDirection(const FString& InString)
{
	return ComputeTextDirection(*InString, 0, InString.Len());
}

ETextDirection ComputeTextDirection(const TCHAR* InString, const int32 InStringStartIndex, const int32 InStringLen)
{
	if (InStringLen == 0)
	{
		return ETextDirection::LeftToRight;
	}

	icu::UnicodeString ICUString = ICUUtilities::ConvertString(InString, InStringStartIndex, InStringLen);

	UErrorCode ICUStatus = U_ZERO_ERROR;
	UBiDi* ICUBiDi = ubidi_openSized(ICUString.length(), 0, &ICUStatus);
	if (ICUBiDi && U_SUCCESS(ICUStatus))
	{
		const ETextDirection ReturnDirection = Internal::ComputeTextDirection(ICUBiDi, ICUString);

		ubidi_close(ICUBiDi);
		ICUBiDi = nullptr;

		return ReturnDirection;
	}
	else
	{
		UE_LOGF(LogCore, Warning, "Failed to create ICU BiDi object (error code: %d). Text will assumed to be left-to-right", static_cast<int32>(ICUStatus));
	}

	return ETextDirection::LeftToRight;
}

ETextDirection ComputeTextDirection(const FText& InText, const ETextDirection InBaseDirection, TArray<FTextDirectionInfo>& OutTextDirectionInfo)
{
	return ComputeTextDirection(InText.ToString(), InBaseDirection, OutTextDirectionInfo);
}

ETextDirection ComputeTextDirection(const FString& InString, const ETextDirection InBaseDirection, TArray<FTextDirectionInfo>& OutTextDirectionInfo)
{
	return ComputeTextDirection(*InString, 0, InString.Len(), InBaseDirection, OutTextDirectionInfo);
}

ETextDirection ComputeTextDirection(const TCHAR* InString, const int32 InStringStartIndex, const int32 InStringLen, const ETextDirection InBaseDirection, TArray<FTextDirectionInfo>& OutTextDirectionInfo)
{
	OutTextDirectionInfo.Reset();

	if (InStringLen == 0)
	{
		return ETextDirection::LeftToRight;
	}

	icu::UnicodeString ICUString = ICUUtilities::ConvertString(InString, InStringStartIndex, InStringLen);

	UErrorCode ICUStatus = U_ZERO_ERROR;
	UBiDi* ICUBiDi = ubidi_openSized(ICUString.length(), 0, &ICUStatus);
	if (ICUBiDi && U_SUCCESS(ICUStatus))
	{
		const ETextDirection ReturnDirection = Internal::ComputeTextDirection(ICUBiDi, ICUString, InStringStartIndex, InBaseDirection, OutTextDirectionInfo);

		ubidi_close(ICUBiDi);
		ICUBiDi = nullptr;

		return ReturnDirection;
	}
	else
	{
		UE_LOGF(LogCore, Warning, "Failed to create ICU BiDi object (error code: %d). Text will assumed to be left-to-right", static_cast<int32>(ICUStatus));
	}

	return ETextDirection::LeftToRight;
}

ETextDirection ComputeBaseDirection(const FText& InText)
{
	return ComputeBaseDirection(InText.ToString());
}

ETextDirection ComputeBaseDirection(const FString& InString)
{
	return ComputeBaseDirection(*InString, 0, InString.Len());
}

ETextDirection ComputeBaseDirection(const TCHAR* InString, const int32 InStringStartIndex, const int32 InStringLen)
{
	if (InStringLen == 0)
	{
		return ETextDirection::LeftToRight;
	}

	icu::UnicodeString ICUString = ICUUtilities::ConvertString(InString, InStringStartIndex, InStringLen);

	return Internal::ComputeBaseDirection(ICUString);
}

} // namespace TextBiDi

#endif
