// Copyright Epic Games, Inc. All Rights Reserved.

#include "Internationalization/ICUTextCharacterIterator.h"
#include "Internationalization/Text.h"

#if UE_ENABLE_ICU

#include "Internationalization/ICUUtilities.h"

UOBJECT_DEFINE_RTTI_IMPLEMENTATION(FICUTextCharacterIterator_NativeUTF16)

FICUTextCharacterIterator_NativeUTF16::FICUTextCharacterIterator_NativeUTF16(FString&& InString)
	: InternalString(MoveTemp(InString))
	, StringRef(InternalString)
{
	SetTextFromStringRef();
}

FICUTextCharacterIterator_NativeUTF16::FICUTextCharacterIterator_NativeUTF16(FStringView InString)
	: StringRef(InString)
{
	SetTextFromStringRef();
}

FICUTextCharacterIterator_NativeUTF16::FICUTextCharacterIterator_NativeUTF16(const FICUTextCharacterIterator_NativeUTF16& Other)
	: icu::UCharCharacterIterator(Other)
	, InternalString(FString::ConstructFromPtrSize(Other.StringRef.GetData(), Other.StringRef.Len()))
	, StringRef(InternalString)
{
	SetTextFromStringRef();
}

FICUTextCharacterIterator_NativeUTF16::~FICUTextCharacterIterator_NativeUTF16()
{
}

int32 FICUTextCharacterIterator_NativeUTF16::InternalIndexToSourceIndex(const int32 InInternalIndex) const
{
	// If the UTF-16 variant is being used, then FString must be UTF-16 so no conversion is required
	return InInternalIndex;
}

int32 FICUTextCharacterIterator_NativeUTF16::SourceIndexToInternalIndex(const int32 InSourceIndex) const
{
	// If the UTF-16 variant is being used, then FString must be UTF-16 so no conversion is required
	return InSourceIndex;
}

// ICU 78 changed UCharCharacterIterator::clone() to return UCharCharacterIterator* instead of CharacterIterator*.
// Our override follows the same covariant pattern, returning our concrete type.
#if WITH_ICU_V78
FICUTextCharacterIterator_NativeUTF16* FICUTextCharacterIterator_NativeUTF16::clone() const
#else
icu::CharacterIterator* FICUTextCharacterIterator_NativeUTF16::clone() const
#endif
{
	return new FICUTextCharacterIterator_NativeUTF16(*this);
}

void FICUTextCharacterIterator_NativeUTF16::SetTextFromStringRef()
{
	setText(reinterpret_cast<const UChar*>(StringRef.GetData()), StringRef.Len()); // scary cast from TCHAR* to UChar* so that this builds on platforms where TCHAR isn't UTF-16 (but we won't use it!)
}


UOBJECT_DEFINE_RTTI_IMPLEMENTATION(FICUTextCharacterIterator_ConvertToUnicodeString)

FICUTextCharacterIterator_ConvertToUnicodeStringPrivate::FICUTextCharacterIterator_ConvertToUnicodeStringPrivate(FString&& InString)
	: SourceString(MoveTemp(InString))
	, InternalString(ICUUtilities::ConvertString(SourceString))
{
}

FICUTextCharacterIterator_ConvertToUnicodeStringPrivate::FICUTextCharacterIterator_ConvertToUnicodeStringPrivate(const FICUTextCharacterIterator_ConvertToUnicodeStringPrivate& Other)
	: SourceString(Other.SourceString)
	, InternalString(Other.InternalString)
{
}

FICUTextCharacterIterator_ConvertToUnicodeString::FICUTextCharacterIterator_ConvertToUnicodeString(FString&& InString)
	: FICUTextCharacterIterator_ConvertToUnicodeStringPrivate(MoveTemp(InString))
	, icu::StringCharacterIterator(InternalString)
{
}

FICUTextCharacterIterator_ConvertToUnicodeString::FICUTextCharacterIterator_ConvertToUnicodeString(FStringView InString)
	: FICUTextCharacterIterator_ConvertToUnicodeStringPrivate(FString::ConstructFromPtrSize(InString.GetData(), InString.Len()))
	, icu::StringCharacterIterator(InternalString)
{
}

FICUTextCharacterIterator_ConvertToUnicodeString::FICUTextCharacterIterator_ConvertToUnicodeString(const FICUTextCharacterIterator_ConvertToUnicodeString& Other)
	: FICUTextCharacterIterator_ConvertToUnicodeStringPrivate(Other)
	, icu::StringCharacterIterator(Other)
{
}

FICUTextCharacterIterator_ConvertToUnicodeString::~FICUTextCharacterIterator_ConvertToUnicodeString()
{
}

int32 FICUTextCharacterIterator_ConvertToUnicodeString::InternalIndexToSourceIndex(const int32 InInternalIndex) const
{
	// Convert from the ICU UTF-16 index to whatever FString needs
	return InInternalIndex == INDEX_NONE ? INDEX_NONE : ICUUtilities::GetNativeStringLength(InternalString, 0, InInternalIndex);
}

int32 FICUTextCharacterIterator_ConvertToUnicodeString::SourceIndexToInternalIndex(const int32 InSourceIndex) const
{
	// Convert from whatever FString is to ICU UTF-16
	return InSourceIndex == INDEX_NONE ? INDEX_NONE : ICUUtilities::GetUnicodeStringLength(*SourceString, 0, InSourceIndex);
}

// ICU 78 changed StringCharacterIterator::clone() to return StringCharacterIterator* instead of CharacterIterator*.
// Our override follows the same covariant pattern, returning our concrete type.
#if WITH_ICU_V78
FICUTextCharacterIterator_ConvertToUnicodeString* FICUTextCharacterIterator_ConvertToUnicodeString::clone() const
#else
icu::CharacterIterator* FICUTextCharacterIterator_ConvertToUnicodeString::clone() const
#endif
{
	return new FICUTextCharacterIterator_ConvertToUnicodeString(*this);
}

#endif
