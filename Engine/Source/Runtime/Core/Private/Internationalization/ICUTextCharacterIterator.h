// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include "ICUUtilities.h"

#if UE_ENABLE_ICU

THIRD_PARTY_INCLUDES_START
	#include <unicode/uchriter.h>
	#include <unicode/schriter.h>
THIRD_PARTY_INCLUDES_END

/**
 * Implementation of an ICU CharacterIterator that is able to iterate over an FText/FString directly since the native string format for this platform is already UTF-16 (as used by ICU)
 * This can be used with the ICU break iterator types by passing it in via adoptText(...)
 * Note: Do not use this type directly! Use the FICUTextCharacterIterator typedef, which will be set correctly for your platform
 */
class FICUTextCharacterIterator_NativeUTF16 : public icu::UCharCharacterIterator
{
public:
	explicit FICUTextCharacterIterator_NativeUTF16(FString&& InString);
	explicit FICUTextCharacterIterator_NativeUTF16(FStringView InString);
	virtual ~FICUTextCharacterIterator_NativeUTF16();

	int32 InternalIndexToSourceIndex(const int32 InInternalIndex) const;
	int32 SourceIndexToInternalIndex(const int32 InSourceIndex) const;

	// ICU RTTI
	static UClassID getStaticClassID();
	virtual UClassID getDynamicClassID() const override;

	// Implement icu::CharacterIterator
#if WITH_ICU_V78
	virtual FICUTextCharacterIterator_NativeUTF16* clone() const override;
#else
	virtual icu::CharacterIterator* clone() const override;
#endif

private:
	FICUTextCharacterIterator_NativeUTF16(const FICUTextCharacterIterator_NativeUTF16& Other);
	FICUTextCharacterIterator_NativeUTF16& operator=(const FICUTextCharacterIterator_NativeUTF16&) = delete;

	void SetTextFromStringRef();

	/** Internal copy of the string (if constructed by value/move). Do not access directly, use StringRef instead */
	FString InternalString;

	/** View to either InternalString, or the external string we were told to reference */
	FStringView StringRef;
};

/**
 * Private implemenation type used by FICUTextCharacterIterator_ConvertToUnicodeString to avoid copying an icu::UnicodeString twice
 * (once to construct the icu::StringCharacterIterator, and once to get hold of the string again)
 * With this, the private type is initialized first (populating the InternalString), which is then copied once into the icu::StringCharacterIterator
 */
class FICUTextCharacterIterator_ConvertToUnicodeStringPrivate
{
public:
	explicit FICUTextCharacterIterator_ConvertToUnicodeStringPrivate(FString&& InString);

protected:
	FICUTextCharacterIterator_ConvertToUnicodeStringPrivate(const FICUTextCharacterIterator_ConvertToUnicodeStringPrivate& Other);
	FICUTextCharacterIterator_ConvertToUnicodeStringPrivate& operator=(const FICUTextCharacterIterator_ConvertToUnicodeStringPrivate&) = delete;

	/** Original source string */
	FString SourceString;

	/** Internal ICU string */
	icu::UnicodeString InternalString;
};

/**
 * Implementation of an ICU CharacterIterator that converts an FText/FString to an icu::UnicodeString, since the native string format for this platform is not UTF-16 (as used by ICU)
 * This can be used with the ICU break iterator types by passing it in via adoptText(...)
 * Note: Do not use this type directly! Use the FICUTextCharacterIterator typedef, which will be set correctly for your platform
 */
class FICUTextCharacterIterator_ConvertToUnicodeString : private FICUTextCharacterIterator_ConvertToUnicodeStringPrivate, public icu::StringCharacterIterator
{
public:
	explicit FICUTextCharacterIterator_ConvertToUnicodeString(FString&& InString);
	explicit FICUTextCharacterIterator_ConvertToUnicodeString(FStringView InString);
	virtual ~FICUTextCharacterIterator_ConvertToUnicodeString();

	int32 InternalIndexToSourceIndex(const int32 InInternalIndex) const;
	int32 SourceIndexToInternalIndex(const int32 InSourceIndex) const;

	// ICU RTTI
	static UClassID getStaticClassID();
	virtual UClassID getDynamicClassID() const override;

	// Implement icu::CharacterIterator
#if WITH_ICU_V78
	virtual FICUTextCharacterIterator_ConvertToUnicodeString* clone() const override;
#else
	virtual icu::CharacterIterator* clone() const override;
#endif

private:
	FICUTextCharacterIterator_ConvertToUnicodeString(const FICUTextCharacterIterator_ConvertToUnicodeString& Other);
	FICUTextCharacterIterator_ConvertToUnicodeString& operator=(const FICUTextCharacterIterator_ConvertToUnicodeString&) = delete;
};

/** Work out the best character iterator to use based upon our native platform string traits */
template <bool IsUnicode, size_t TCHARSize> struct FICUTextCharacterIterator_PlatformSpecific { typedef FICUTextCharacterIterator_ConvertToUnicodeString Type; };
template <> struct FICUTextCharacterIterator_PlatformSpecific<true, 2> { typedef FICUTextCharacterIterator_NativeUTF16 Type; }; // A unicode encoding with a wchar_t size of 2 bytes is assumed to be UTF-16
typedef FICUTextCharacterIterator_PlatformSpecific<FPlatformString::IsUnicodeEncoded, sizeof(TCHAR)>::Type FICUTextCharacterIterator;

#endif
