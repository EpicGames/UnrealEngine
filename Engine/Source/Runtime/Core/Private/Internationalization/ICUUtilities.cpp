// Copyright Epic Games, Inc. All Rights Reserved.

#include "Internationalization/ICUUtilities.h"

#if UE_ENABLE_ICU
THIRD_PARTY_INCLUDES_START
	#include <unicode/ucnv.h>
THIRD_PARTY_INCLUDES_END

namespace ICUUtilities
{
	void FStringConverterImpl_NativeUTF16::ConvertString(const TCHAR* Source, const int32 SourceStartIndex, const int32 SourceLen, icu::UnicodeString& Destination, const bool ShouldNullTerminate)
	{
		if (SourceLen > 0)
		{
			// Get the internal buffer of the string, we're going to write to it directly
			UChar* InternalStringBuffer = Destination.getBuffer(SourceLen + 1);

			// Copy into the string buffer and optionally terminate
			FMemory::Memcpy(InternalStringBuffer, Source + SourceStartIndex, SourceLen * sizeof(TCHAR));
			if (ShouldNullTerminate)
			{
				InternalStringBuffer[SourceLen] = 0;
			}

			// Size it back down to the correct size and release our lock on the string buffer
			Destination.releaseBuffer(SourceLen);
		}
		else
		{
			Destination.remove();
		}
	}

	void FStringConverterImpl_NativeUTF16::ConvertString(const icu::UnicodeString& Source, const int32 SourceStartIndex, const int32 SourceLen, FString& Destination)
	{
		if (SourceLen > 0)
		{
			// Get the internal buffer of the string, we're going to write to it directly
			TArray<TCHAR, FString::AllocatorType>& InternalStringBuffer = Destination.GetCharArray();
			InternalStringBuffer.SetNumUninitialized(SourceLen + 1);

			// Copy into the string buffer and terminate
			FMemory::Memcpy(InternalStringBuffer.GetData(), Source.getBuffer() + SourceStartIndex, SourceLen * sizeof(TCHAR));
			InternalStringBuffer[SourceLen] = 0;
		}
		else
		{
			Destination.Reset();
		}
	}

	FStringConverterImpl_ConvertToUnicodeString::FStringConverterImpl_ConvertToUnicodeString()
		: ICUConverter(nullptr)
	{
		UErrorCode ICUStatus = U_ZERO_ERROR;
		ICUConverter = ucnv_open(FPlatformString::GetEncodingName(), &ICUStatus);
		check(U_SUCCESS(ICUStatus));
	}

	FStringConverterImpl_ConvertToUnicodeString::~FStringConverterImpl_ConvertToUnicodeString()
	{
		ucnv_close(ICUConverter);
	}

	void FStringConverterImpl_ConvertToUnicodeString::ConvertString(const TCHAR* Source, const int32 SourceStartIndex, const int32 SourceLen, icu::UnicodeString& Destination, const bool ShouldNullTerminate)
	{
		if (SourceLen > 0)
		{
			UErrorCode ICUStatus = U_ZERO_ERROR;

			ucnv_reset(ICUConverter);

			// Get the internal buffer of the string, we're going to use it as scratch space
			const int32_t DestinationCapacityUChars = SourceLen * 2;
			UChar* InternalStringBuffer = Destination.getBuffer(DestinationCapacityUChars);

			// Perform the conversion into the string buffer
			const int32_t SourceSizeBytes = SourceLen * sizeof(TCHAR);
			const int32_t DestinationLength = ucnv_toUChars(ICUConverter, InternalStringBuffer, DestinationCapacityUChars, reinterpret_cast<const char*>(Source + SourceStartIndex), SourceSizeBytes, &ICUStatus);

			// Optionally null terminate the string
			if (ShouldNullTerminate)
			{
				InternalStringBuffer[DestinationLength] = 0;
			}

			// Size it back down to the correct size and release our lock on the string buffer
			Destination.releaseBuffer(DestinationLength);

			check(U_SUCCESS(ICUStatus));
		}
		else
		{
			Destination.remove();
		}
	}

	void FStringConverterImpl_ConvertToUnicodeString::ConvertString(const icu::UnicodeString& Source, const int32 SourceStartIndex, const int32 SourceLen, FString& Destination)
	{
		if (SourceLen > 0)
		{
			UErrorCode ICUStatus = U_ZERO_ERROR;

			ucnv_reset(ICUConverter);

			// Get the internal buffer of the string, we're going to use it as scratch space
			TArray<TCHAR, FString::AllocatorType>& InternalStringBuffer = Destination.GetCharArray();

			// Work out the maximum size required and resize the buffer so it can hold enough data
			const int32_t DestinationCapacityBytes = UCNV_GET_MAX_BYTES_FOR_STRING(SourceLen, ucnv_getMaxCharSize(ICUConverter));
			const int32 DestinationCapacityTCHARs = DestinationCapacityBytes / sizeof(TCHAR);
			InternalStringBuffer.SetNumUninitialized(DestinationCapacityTCHARs);

			// Perform the conversion into the string buffer, and then null terminate the FString and size it back down to the correct size
			const int32_t DestinationSizeBytes = ucnv_fromUChars(ICUConverter, reinterpret_cast<char*>(InternalStringBuffer.GetData()), DestinationCapacityBytes, Source.getBuffer() + SourceStartIndex, SourceLen, &ICUStatus);
			const int32 DestinationSizeTCHARs = DestinationSizeBytes / sizeof(TCHAR);
			InternalStringBuffer[DestinationSizeTCHARs] = 0;
			InternalStringBuffer.SetNum(DestinationSizeTCHARs + 1, EAllowShrinking::No); // the array size includes null

			check(U_SUCCESS(ICUStatus));
		}
		else
		{
			Destination.Reset();
		}
	}

	void FStringConverter::ConvertString(FStringView Source, icu::UnicodeString& Destination, const bool ShouldNullTerminate)
	{
		ConvertString(Source.GetData(), 0, Source.Len(), Destination, ShouldNullTerminate);
	}

	void FStringConverter::ConvertString(const TCHAR* Source, const int32 SourceStartIndex, const int32 SourceLen, icu::UnicodeString& Destination, const bool ShouldNullTerminate)
	{
		Impl.ConvertString(Source, SourceStartIndex, SourceLen, Destination, ShouldNullTerminate);
	}

	icu::UnicodeString FStringConverter::ConvertString(FStringView Source, const bool ShouldNullTerminate)
	{
		icu::UnicodeString Destination;
		ConvertString(Source, Destination, ShouldNullTerminate);
		return Destination;
	}

	icu::UnicodeString FStringConverter::ConvertString(const TCHAR* Source, const int32 SourceStartIndex, const int32 SourceLen, const bool ShouldNullTerminate)
	{
		icu::UnicodeString Destination;
		ConvertString(Source, SourceStartIndex, SourceLen, Destination, ShouldNullTerminate);
		return Destination;
	}

	void FStringConverter::ConvertString(const icu::UnicodeString& Source, FString& Destination)
	{
		return ConvertString(Source, 0, Source.length(), Destination);
	}

	void FStringConverter::ConvertString(const icu::UnicodeString& Source, const int32 SourceStartIndex, const int32 SourceLen, FString& Destination)
	{
		Impl.ConvertString(Source, SourceStartIndex, SourceLen, Destination);
	}

	FString FStringConverter::ConvertString(const icu::UnicodeString& Source)
	{
		FString Destination;
		ConvertString(Source, Destination);
		return Destination;
	}

	FString FStringConverter::ConvertString(const icu::UnicodeString& Source, const int32 SourceStartIndex, const int32 SourceLen)
	{
		FString Destination;
		ConvertString(Source, SourceStartIndex, SourceLen, Destination);
		return Destination;
	}

	void ConvertString(FStringView Source, icu::UnicodeString& Destination, const bool ShouldNullTerminate)
	{
		if (Source.Len() > 0)
		{
			FStringConverter StringConverter;
			StringConverter.ConvertString(Source, Destination, ShouldNullTerminate);
		}
		else
		{
			Destination.remove();
		}
	}

	void ConvertString(const TCHAR* Source, const int32 SourceStartIndex, const int32 SourceLen, icu::UnicodeString& Destination, const bool ShouldNullTerminate)
	{
		if (SourceLen > 0)
		{
			FStringConverter StringConverter;
			StringConverter.ConvertString(Source, SourceStartIndex, SourceLen, Destination, ShouldNullTerminate);
		}
		else
		{
			Destination.remove();
		}
	}

	icu::UnicodeString ConvertString(FStringView Source, const bool ShouldNullTerminate)
	{
		icu::UnicodeString Destination;
		ConvertString(Source, Destination, ShouldNullTerminate);
		return Destination;
	}

	icu::UnicodeString ConvertString(const TCHAR* Source, const int32 SourceStartIndex, const int32 SourceLen, const bool ShouldNullTerminate)
	{
		icu::UnicodeString Destination;
		ConvertString(Source, SourceStartIndex, SourceLen, Destination, ShouldNullTerminate);
		return Destination;
	}

	void ConvertString(const icu::UnicodeString& Source, FString& Destination)
	{
		return ConvertString(Source, 0, Source.length(), Destination);
	}

	void ConvertString(const icu::UnicodeString& Source, const int32 SourceStartIndex, const int32 SourceLen, FString& Destination)
	{
		if (SourceLen)
		{
			FStringConverter StringConverter;
			StringConverter.ConvertString(Source, SourceStartIndex, SourceLen, Destination);
		}
		else
		{
			Destination.Empty();
		}
	}

	FString ConvertString(const icu::UnicodeString& Source)
	{
		FString Destination;
		ConvertString(Source, Destination);
		return Destination;
	}

	FString ConvertString(const icu::UnicodeString& Source, const int32 SourceStartIndex, const int32 SourceLen)
	{
		FString Destination;
		ConvertString(Source, SourceStartIndex, SourceLen, Destination);
		return Destination;
	}

	template <bool IsUnicode, size_t TCHARSize>
	int32 GetNativeStringLengthImpl(const icu::UnicodeString& Source, const int32 InSourceStartIndex, const int32 InSourceLength)
	{
		if (InSourceLength > 0)
		{
			const FString TmpStr = ConvertString(Source, InSourceStartIndex, InSourceLength);
			return TmpStr.Len();
		}
		return 0;
	}

	// A unicode encoding with a wchar_t size of 2 bytes is assumed to be UTF-16
	template <>
	int32 GetNativeStringLengthImpl<true, 2>(const icu::UnicodeString& Source, const int32 InSourceStartIndex, const int32 InSourceLength)
	{
		// Nothing to do, ICU already uses UTF-16 internally
		return InSourceLength;
	}

	// A unicode encoding with a wchar_t size of 4 bytes is assumed to be UTF-32
	template <>
	int32 GetNativeStringLengthImpl<true, 4>(const icu::UnicodeString& Source, const int32 InSourceStartIndex, const int32 InSourceLength)
	{
		return InSourceLength == 0 ? 0 : Source.countChar32(InSourceStartIndex, InSourceLength);
	}

	int32 GetNativeStringLength(const icu::UnicodeString& Source)
	{
		return GetNativeStringLength(Source, 0, Source.length());
	}

	int32 GetNativeStringLength(const icu::UnicodeString& Source, const int32 InSourceStartIndex, const int32 InSourceLength)
	{
		return GetNativeStringLengthImpl<FPlatformString::IsUnicodeEncoded, sizeof(TCHAR)>(Source, InSourceStartIndex, InSourceLength);
	}

	template <bool IsUnicode, size_t TCHARSize>
	int32 GetUnicodeStringLengthImpl(const TCHAR* Source, const int32 InSourceStartIndex, const int32 InSourceLength)
	{
		if (InSourceLength > 0)
		{
			const icu::UnicodeString TmpStr = ConvertString(Source, InSourceStartIndex, InSourceLength);
			return TmpStr.length();
		}
		return 0;
	}

	// A unicode encoding with a wchar_t size of 2 bytes is assumed to be UTF-16
	template <>
	int32 GetUnicodeStringLengthImpl<true, 2>(const TCHAR* Source, const int32 InSourceStartIndex, const int32 InSourceLength)
	{
		// Nothing to do, ICU already uses UTF-16 internally
		return InSourceLength;
	}

	// A unicode encoding with a wchar_t size of 4 bytes is assumed to be UTF-32
	template <>
	int32 GetUnicodeStringLengthImpl<true, 4>(const TCHAR* Source, const int32 InSourceStartIndex, const int32 InSourceLength)
	{
		int32 Len = 0;
		if (InSourceLength > 0)
		{
			const TCHAR* SourceStartChar = Source + InSourceStartIndex;
			const TCHAR* SourceEndChar = SourceStartChar + InSourceLength;
			for (const TCHAR* SourceChar = SourceStartChar; SourceChar < SourceEndChar; ++SourceChar)
			{
				Len += (*SourceChar > 0xFFFF ? 2 : 1); // 0xFFFF is the largest code-point a single UTF-16 character can hold
			}
		}
		return Len;
	}

	int32 GetUnicodeStringLength(FStringView Source)
	{
		return GetUnicodeStringLength(Source.GetData(), 0, Source.Len());
	}

	int32 GetUnicodeStringLength(const TCHAR* Source, const int32 InSourceStartIndex, const int32 InSourceLength)
	{
		return GetUnicodeStringLengthImpl<FPlatformString::IsUnicodeEncoded, sizeof(TCHAR)>(Source, InSourceStartIndex, InSourceLength);
	}
}
#endif
