// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include "Misc/Timespan.h"
#include "Internationalization/InternationalizationUtilities.h"

#if UE_ENABLE_ICU
THIRD_PARTY_INCLUDES_START
	#include <unicode/unistr.h>
THIRD_PARTY_INCLUDES_END

#ifndef WITH_ICU_V64
	#define WITH_ICU_V64 0
#endif

#ifndef WITH_ICU_V78
	#define WITH_ICU_V78 0
#endif

namespace ICUUtilities
{
	// InternationalizationUtilities was split from ICUUtilities, so import its functions to avoid breaking existing code
	using namespace InternationalizationUtilities;

	/**
	 * Implementation of the string converter that can copy FString to icu::UnicodeString directly since the native string format for this platform is already UTF-16 (as used by ICU)
	 * Note: Do not use this type directly! Use the FStringConverterImpl typedef, which will be set correctly for your platform
	 */
	class FStringConverterImpl_NativeUTF16
	{
	public:
		void ConvertString(const TCHAR* Source, const int32 SourceStartIndex, const int32 SourceLen, icu::UnicodeString& Destination, const bool ShouldNullTerminate);
		void ConvertString(const icu::UnicodeString& Source, const int32 SourceStartIndex, const int32 SourceLen, FString& Destination);
	};

	/**
	 * Implementation of the string converter that can copy FString to icu::UnicodeString via an ICU converter since the native string format for this platform is not UTF-16 (as used by ICU)
	 * Note: Do not use this type directly! Use the FStringConverterImpl typedef, which will be set correctly for your platform
	 */
	class FStringConverterImpl_ConvertToUnicodeString
	{
	public:
		FStringConverterImpl_ConvertToUnicodeString();
		~FStringConverterImpl_ConvertToUnicodeString();

		void ConvertString(const TCHAR* Source, const int32 SourceStartIndex, const int32 SourceLen, icu::UnicodeString& Destination, const bool ShouldNullTerminate);
		void ConvertString(const icu::UnicodeString& Source, const int32 SourceStartIndex, const int32 SourceLen, FString& Destination);

	private:
		UConverter* ICUConverter;
	};

	/** Work out the best string converter to use based upon our native platform string traits */
	template <bool IsUnicode, size_t TCHARSize> struct FStringConverterImpl_PlatformSpecific { typedef FStringConverterImpl_ConvertToUnicodeString Type; };
	template <> struct FStringConverterImpl_PlatformSpecific<true, 2> { typedef FStringConverterImpl_NativeUTF16 Type; }; // A unicode encoding with a wchar_t size of 2 bytes is assumed to be UTF-16
	typedef FStringConverterImpl_PlatformSpecific<FPlatformString::IsUnicodeEncoded, sizeof(TCHAR)>::Type FStringConverterImpl;

	/**
	 * An object that can convert between FString and icu::UnicodeString
	 * Note: This object is not thread-safe.
	 */
	class FStringConverter
	{
	public:
		FStringConverter() = default;
		FStringConverter(const FStringConverter&) = delete;
		FStringConverter& operator=(const FStringConverter&) = delete;

		/** Convert FString -> icu::UnicodeString */
		void ConvertString(FStringView Source, icu::UnicodeString& Destination, const bool ShouldNullTerminate = true);
		void ConvertString(const TCHAR* Source, const int32 SourceStartIndex, const int32 SourceLen, icu::UnicodeString& Destination, const bool ShouldNullTerminate = true);
		icu::UnicodeString ConvertString(FStringView Source, const bool ShouldNullTerminate = true);
		icu::UnicodeString ConvertString(const TCHAR* Source, const int32 SourceStartIndex, const int32 SourceLen, const bool ShouldNullTerminate = true);

		/** Convert icu::UnicodeString -> FString */
		void ConvertString(const icu::UnicodeString& Source, FString& Destination);
		void ConvertString(const icu::UnicodeString& Source, const int32 SourceStartIndex, const int32 SourceLen, FString& Destination);
		FString ConvertString(const icu::UnicodeString& Source);
		FString ConvertString(const icu::UnicodeString& Source, const int32 SourceStartIndex, const int32 SourceLen);

	private:
		FStringConverterImpl Impl;
	};

	/** Convert FString -> icu::UnicodeString */
	void ConvertString(FStringView Source, icu::UnicodeString& Destination, const bool ShouldNullTerminate = true);
	void ConvertString(const TCHAR* Source, const int32 SourceStartIndex, const int32 SourceLen, icu::UnicodeString& Destination, const bool ShouldNullTerminate = true);
	icu::UnicodeString ConvertString(FStringView Source, const bool ShouldNullTerminate = true);
	icu::UnicodeString ConvertString(const TCHAR* Source, const int32 SourceStartIndex, const int32 SourceLen, const bool ShouldNullTerminate = true);

	/** Convert icu::UnicodeString -> FString */
	void ConvertString(const icu::UnicodeString& Source, FString& Destination);
	void ConvertString(const icu::UnicodeString& Source, const int32 SourceStartIndex, const int32 SourceLen, FString& Destination);
	FString ConvertString(const icu::UnicodeString& Source);
	FString ConvertString(const icu::UnicodeString& Source, const int32 SourceStartIndex, const int32 SourceLen);

	/** Given an icu::UnicodeString, count how many characters it would be if converted into an FString (as FString may not always be UTF-16) */
	int32 GetNativeStringLength(const icu::UnicodeString& Source);
	int32 GetNativeStringLength(const icu::UnicodeString& Source, const int32 InSourceStartIndex, const int32 InSourceLength);

	/** Given an FString, count how many characters it would be if converted to an icu::UnicodeString (as FString may not always be UTF-16) */
	int32 GetUnicodeStringLength(FStringView Source);
	int32 GetUnicodeStringLength(const TCHAR* Source, const int32 InSourceStartIndex, const int32 InSourceLength);
}
#endif
