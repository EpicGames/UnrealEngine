// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "Templates/SharedPointer.h"
#include "Internationalization/IBreakIterator.h"
#include "Internationalization/BreakIterator.h"

#if UE_ENABLE_ICU
#include "Internationalization/ICUBreakIterator.h"

TSharedRef<IBreakIterator> FBreakIterator::CreateWordBreakIterator()
{
	return MakeShareable(new FICUBreakIterator(FICUBreakIteratorManager::Get().CreateWordBreakIterator()));
}

TSharedRef<IBreakIterator> FBreakIterator::GetWordBreakIterator()
{
	static thread_local TSharedPtr<IBreakIterator> CachedWordBreakIterator;
	if (!CachedWordBreakIterator.IsValid())
	{
		CachedWordBreakIterator = MakeShareable(new FICUBreakIterator(FICUBreakIteratorManager::Get().CreateWordBreakIterator()));
	}

	check(CachedWordBreakIterator.IsValid());
	return CachedWordBreakIterator.ToSharedRef();
}

#endif
