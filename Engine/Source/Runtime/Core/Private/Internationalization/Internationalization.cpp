// Copyright Epic Games, Inc. All Rights Reserved.

#include "Internationalization/Internationalization.h"
#include "Internationalization/TextLocalizationResource.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/FileManager.h"
#include "HAL/LowLevelMemTracker.h"
#include "Misc/LazySingleton.h"
#include "Misc/Paths.h"
#include "Internationalization/Culture.h"
#include "Internationalization/CustomCultureImplementation.h"

#if UE_ENABLE_ICU
#include "Internationalization/ICUInternationalization.h"
#else
#include "Internationalization/LegacyInternationalization.h"
#endif

#define LOCTEXT_NAMESPACE "Internationalization"

FInternationalization& FInternationalization::Get()
{
	return TLazySingleton<FInternationalization>::Get();
}

bool FInternationalization::IsAvailable()
{
	FInternationalization* Singleton = TLazySingleton<FInternationalization>::TryGet();
	return Singleton && Singleton->bIsInitialized;
}

void FInternationalization::TearDown()
{
	TLazySingleton<FInternationalization>::TearDown();
}

bool FInternationalization::SetCurrentCulture(const FString& InCultureName)
{
	FCulturePtr NewCulture = GetCulture(InCultureName);

	if (NewCulture.IsValid())
	{
		if (CurrentLanguage != NewCulture || CurrentLocale != NewCulture || CurrentAssetGroupCultures.Num() > 0)
		{
			CurrentLanguage = NewCulture;
			CurrentLocale = NewCulture;
			CurrentAssetGroupCultures.Reset();
	
			Implementation->HandleLanguageChanged(CurrentLanguage.ToSharedRef());
	
			BroadcastCultureChanged();
		}
	}

	return CurrentLanguage == NewCulture && CurrentLocale == NewCulture && CurrentAssetGroupCultures.Num() == 0;
}

bool FInternationalization::SetCurrentLanguage(const FString& InCultureName)
{
	FCulturePtr NewCulture = GetCulture(InCultureName);

	if (NewCulture.IsValid())
	{
		if (CurrentLanguage != NewCulture)
		{
			CurrentLanguage = NewCulture;

			Implementation->HandleLanguageChanged(CurrentLanguage.ToSharedRef());

			BroadcastCultureChanged();
		}
	}

	return CurrentLanguage == NewCulture;
}

bool FInternationalization::SetCurrentLocale(const FString& InCultureName)
{
	FCulturePtr NewCulture = GetCulture(InCultureName);

	if (NewCulture.IsValid())
	{
		if (CurrentLocale != NewCulture)
		{
			CurrentLocale = NewCulture;

			BroadcastCultureChanged();
		}
	}

	return CurrentLocale == NewCulture;
}

bool FInternationalization::SetCurrentLanguageAndLocale(const FString& InCultureName)
{
	FCulturePtr NewCulture = GetCulture(InCultureName);

	if (NewCulture.IsValid())
	{
		if (CurrentLanguage != NewCulture || CurrentLocale != NewCulture)
		{
			CurrentLanguage = NewCulture;
			CurrentLocale = NewCulture;

			Implementation->HandleLanguageChanged(CurrentLanguage.ToSharedRef());

			BroadcastCultureChanged();
		}
	}

	return CurrentLanguage == NewCulture && CurrentLocale == NewCulture;
}

bool FInternationalization::SetCurrentAssetGroupCulture(const FName& InAssetGroupName, const FString& InCultureName)
{
	FCulturePtr NewCulture = GetCulture(InCultureName);

	if (NewCulture.IsValid())
	{
		TTuple<FName, FCulturePtr>* EntryToUpdate = CurrentAssetGroupCultures.FindByPredicate([InAssetGroupName](const TTuple<FName, FCulturePtr>& InCurrentAssetGroupCulturePair)
		{
			return InCurrentAssetGroupCulturePair.Key == InAssetGroupName;
		});

		bool bChangedCulture = false;
		if (EntryToUpdate)
		{
			if (EntryToUpdate->Value != NewCulture)
			{
				bChangedCulture = true;
				EntryToUpdate->Value = NewCulture;
			}
		}
		else
		{
			bChangedCulture = true;
			CurrentAssetGroupCultures.Add(MakeTuple(InAssetGroupName, NewCulture));
		}

		if (bChangedCulture)
		{
			BroadcastCultureChanged();
		}

		return true;
	}

	return false;
}

FCultureRef FInternationalization::GetCurrentAssetGroupCulture(const FName& InAssetGroupName) const
{
	for (const TTuple<FName, FCulturePtr>& CurrentAssetGroupCulturePair : CurrentAssetGroupCultures)
	{
		if (CurrentAssetGroupCulturePair.Key == InAssetGroupName)
		{
			return CurrentAssetGroupCulturePair.Value.ToSharedRef();
		}
	}
	return GetCurrentLanguage();
}

void FInternationalization::ClearCurrentAssetGroupCulture(const FName& InAssetGroupName)
{
	CurrentAssetGroupCultures.RemoveAll([InAssetGroupName](const TTuple<FName, FCulturePtr>& InCurrentAssetGroupCulturePair)
	{
		return InCurrentAssetGroupCulturePair.Key == InAssetGroupName;
	});
}

TArray<FCultureRef> FInternationalization::GetCurrentCultures(const bool bIncludeLanguage, const bool bIncludeLocale, const bool bIncludeAssetGroups) const
{
	TArray<FCultureRef> CurrentCultures;

	if (bIncludeLanguage)
	{
		CurrentCultures.AddUnique(CurrentLanguage.ToSharedRef());
	}

	if (bIncludeLocale)
	{
		CurrentCultures.AddUnique(CurrentLocale.ToSharedRef());
	}

	if (bIncludeAssetGroups)
	{
		for (const TTuple<FName, FCulturePtr>& CurrentAssetGroupCulturePair : CurrentAssetGroupCultures)
		{
			CurrentCultures.AddUnique(CurrentAssetGroupCulturePair.Value.ToSharedRef());
		}
	}

	return CurrentCultures;
}

void FInternationalization::BackupCultureState(FCultureStateSnapshot& OutSnapshot) const
{
	OutSnapshot.Language = CurrentLanguage->GetName();
	OutSnapshot.Locale = CurrentLocale->GetName();

	OutSnapshot.AssetGroups.Reset(CurrentAssetGroupCultures.Num());
	for (const TTuple<FName, FCulturePtr>& CurrentAssetGroupCulturePair : CurrentAssetGroupCultures)
	{
		OutSnapshot.AssetGroups.Add(MakeTuple(CurrentAssetGroupCulturePair.Key, CurrentAssetGroupCulturePair.Value->GetName()));
	}
}

void FInternationalization::RestoreCultureState(const FCultureStateSnapshot& InSnapshot)
{
	bool bChangedCulture = false;

	// Apply the language
	if (!InSnapshot.Language.IsEmpty())
	{
		FCulturePtr NewCulture = GetCulture(InSnapshot.Language);

		if (NewCulture.IsValid())
		{
			if (CurrentLanguage != NewCulture)
			{
				bChangedCulture = true;

				CurrentLanguage = NewCulture;

				Implementation->HandleLanguageChanged(CurrentLanguage.ToSharedRef());
			}
		}
	}

	// Apply the locale
	if (!InSnapshot.Locale.IsEmpty())
	{
		FCulturePtr NewCulture = GetCulture(InSnapshot.Locale);

		if (NewCulture.IsValid())
		{
			if (CurrentLocale != NewCulture)
			{
				bChangedCulture = true;

				CurrentLocale = NewCulture;
			}
		}
	}

	// Apply the asset groups
	bChangedCulture |= CurrentAssetGroupCultures.Num() > 0;
	CurrentAssetGroupCultures.Reset(InSnapshot.AssetGroups.Num());
	for (const auto& AssetGroupCultureNamePair : InSnapshot.AssetGroups)
	{
		FCulturePtr NewCulture = GetCulture(AssetGroupCultureNamePair.Value);
		if (NewCulture.IsValid())
		{
			bChangedCulture = true;
			CurrentAssetGroupCultures.Add(MakeTuple(AssetGroupCultureNamePair.Key, NewCulture));
		}
	}

	if (bChangedCulture)
	{
		BroadcastCultureChanged();
	}
}

FCulturePtr FInternationalization::GetCulture(const FString& InCultureName)
{
	return Implementation->GetCulture(InCultureName);
}

void FInternationalization::Initialize()
{
	static bool IsInitializing = false;

	if(IsInitialized() || IsInitializing)
	{
		return;
	}
	struct FInitializingGuard
	{
		FInitializingGuard()	{IsInitializing = true;}
		~FInitializingGuard()	{IsInitializing = false;}
	} InitializingGuard;

	bIsInitialized = Implementation->Initialize();
}

void FInternationalization::Terminate()
{
	CurrentLanguage.Reset();
	CurrentLocale.Reset();
	CurrentAssetGroupCultures.Reset();

	DefaultLanguage.Reset();
	DefaultLocale.Reset();

	CustomCultures.Reset();
	InvariantCulture.Reset();

	Implementation->Terminate();

	bIsInitialized = false;
}

#if ENABLE_LOC_TESTING
FString& FInternationalization::Leetify(FString& SourceString)
{
	static const TCHAR LeetifyTextStartMarker = TEXT('\x2021');
	static const TCHAR LeetifyTextEndMarker = TEXT('\x2021');
	static const TCHAR LeetifyArgumentStartMarker = TEXT('\x00AB');
	static const TCHAR LeetifyArgumentEndMarker = TEXT('\x00BB');
	static const TCHAR SourceArgumentStartMarker = TEXT('{');
	static const TCHAR SourceArgumentEndMarker = TEXT('}');
	static const TCHAR SourceEscapeMarker = TEXT('`');

	auto LeetifyCharacter = [](const TCHAR InChar) -> TCHAR
	{
		switch (InChar)
		{
		case TEXT('A'): return TEXT('4');
		case TEXT('a'): return TEXT('@');
		case TEXT('B'): return TEXT('8');
		case TEXT('b'): return TEXT('8');
		case TEXT('E'): return TEXT('3');
		case TEXT('e'): return TEXT('3');
		case TEXT('G'): return TEXT('9');
		case TEXT('g'): return TEXT('9');
		case TEXT('I'): return TEXT('1');
		case TEXT('i'): return TEXT('!');
		case TEXT('O'): return TEXT('0');
		case TEXT('o'): return TEXT('0');
		case TEXT('S'): return TEXT('5');
		case TEXT('s'): return TEXT('$');
		case TEXT('T'): return TEXT('7');
		case TEXT('t'): return TEXT('7');
		case TEXT('Z'): return TEXT('2');
		case TEXT('z'): return TEXT('2');
		default:		return InChar;
		}
	};

	if (SourceString.IsEmpty() || (SourceString.Len() >= 2 && SourceString[0] == LeetifyTextStartMarker && SourceString[SourceString.Len() - 1] == LeetifyTextEndMarker))
	{
		// Already leetified
		return SourceString;
	}

	// We insert a start and end marker (+2), and format strings typically have <= 8 argument blocks which we'll wrap with a start and end marker (+16), so +18 should be a reasonable slack
	FString LeetifiedString;
	LeetifiedString.Reserve(SourceString.Len() + 18);

	// Inject the start marker
	LeetifiedString.AppendChar(LeetifyTextStartMarker);

	// Iterate and leetify each character in the source string, but don't change argument names as that will break formatting
	{
		bool bEscapeNextChar = false;

		const int32 SourceStringLen = SourceString.Len();
		for (int32 SourceCharIndex = 0; SourceCharIndex < SourceStringLen; ++SourceCharIndex)
		{
			const TCHAR SourceChar = SourceString[SourceCharIndex];

			if (!bEscapeNextChar && SourceChar == SourceArgumentStartMarker)
			{
				const TCHAR* RawSourceStringPtr = SourceString.GetCharArray().GetData();

				// Walk forward to find the end of this argument block to make sure we have a pair of tokens
				const TCHAR* ArgumentEndPtr = FCString::Strchr(RawSourceStringPtr + SourceCharIndex + 1, SourceArgumentEndMarker);
				if (ArgumentEndPtr)
				{
					const int32 ArgumentEndIndex = UE_PTRDIFF_TO_INT32(ArgumentEndPtr - RawSourceStringPtr);

					// Inject a marker before the argument block
					LeetifiedString.AppendChar(LeetifyArgumentStartMarker);

					// Copy the body of the argument, including the opening and closing tags
					check(ArgumentEndIndex >= SourceCharIndex);
					LeetifiedString.AppendChars(RawSourceStringPtr + SourceCharIndex, (ArgumentEndIndex - SourceCharIndex) + 1);

					// Inject a marker after the end of the argument block
					LeetifiedString.AppendChar(LeetifyArgumentEndMarker);

					// Move to the end of the argument we just parsed
					SourceCharIndex = ArgumentEndIndex;
					continue;
				}
			}

			if (SourceChar == SourceEscapeMarker)
			{
				bEscapeNextChar = !bEscapeNextChar;
			}
			else
			{
				bEscapeNextChar = false;
			}

			LeetifiedString.AppendChar(LeetifyCharacter(SourceChar));
		}
	}

	// Inject the end marker
	LeetifiedString.AppendChar(LeetifyTextEndMarker);

	SourceString = LeetifiedString;
	return SourceString;
}
#endif

void FInternationalization::LoadAllCultureData()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FInternationalization::LoadAllCultureData);
	Implementation->LoadAllCultureData();
}

void FInternationalization::AddCustomCulture(const TSharedRef<ICustomCulture>& InCustomCulture)
{
	CustomCultures.Add(FCulture::Create(MakeUnique<FCustomCultureImplementation>(InCustomCulture)));
}

FCulturePtr FInternationalization::GetCustomCulture(const FString& InCultureName) const
{
	const FCultureRef* CulturePtr = CustomCultures.FindByPredicate([&InCultureName](const FCultureRef& InPotentialCulture)
	{
		return InPotentialCulture->GetName() == InCultureName;
	});
	return CulturePtr ? FCulturePtr(*CulturePtr) : FCulturePtr();
}

bool FInternationalization::IsCultureRemapped(const FString& Name, FString* OutMappedCulture)
{
	return Implementation->IsCultureRemapped(Name, OutMappedCulture);
}

bool FInternationalization::IsCultureAllowed(const FString& Name)
{
	return Implementation->IsCultureAllowed(Name);
}

void FInternationalization::RefreshCultureDisplayNames(const TArray<FString>& InPrioritizedDisplayCultureNames)
{
	Implementation->RefreshCultureDisplayNames(InPrioritizedDisplayCultureNames);
}

void FInternationalization::RefreshCachedConfigData()
{
	Implementation->RefreshCachedConfigData();
}

void FInternationalization::GetCultureNames(TArray<FString>& CultureNames) const
{
	Implementation->GetCultureNames(CultureNames);
}

TArray<FString> FInternationalization::GetPrioritizedCultureNames(const FString& Name)
{
	return Implementation->GetPrioritizedCultureNames(Name);
}

TArray<FCultureRef> FInternationalization::GetAvailableCultures(const TArray<FString>& InCultureNames, const bool bIncludeDerivedCultures)
{
	TArray<FCultureRef> AvailableCultures;

	// Find any cultures that are a partial match for those we have translations for.
	if (bIncludeDerivedCultures)
	{
		TArray<FString> CultureNames;
		GetCultureNames(CultureNames);
		for (const FString& CultureName : CultureNames)
		{
			FCulturePtr Culture = GetCulture(CultureName);
			if (Culture.IsValid())
			{
				const TArray<FString> PrioritizedParentCultureNames = Culture->GetPrioritizedParentCultureNames();
				for (const FString& PrioritizedParentCultureName : PrioritizedParentCultureNames)
				{
					if (InCultureNames.Contains(PrioritizedParentCultureName) && IsCultureAllowed(Culture->GetName()))
					{
						AvailableCultures.AddUnique(Culture.ToSharedRef());
						break;
					}
				}
			}
		}
	}
	// Find any cultures that are a complete match for those we have translations for.
	else
	{
		for (const FString& CultureNames : InCultureNames)
		{
			FCulturePtr Culture = GetCulture(CultureNames);
			if (Culture.IsValid())
			{
				AvailableCultures.AddUnique(Culture.ToSharedRef());
			}
		}
	}

	return AvailableCultures;
}

FInternationalization::FInternationalization()
	:	Implementation(this)
{
	Initialize();
}

FInternationalization::~FInternationalization()
{
	if (IsInitialized())
	{
		Terminate();
	}
}

#undef LOCTEXT_NAMESPACE
