// Copyright Epic Games, Inc. All Rights Reserved.

#include "Internationalization/InternationalizationArchive.h"

#include "Containers/Array.h"
#include "Internationalization/InternationalizationMetadata.h"
#include "Logging/LogMacros.h"

DEFINE_LOG_CATEGORY_STATIC(LogGatherArchiveObject, Log, All);

FArchiveEntry::FArchiveEntry(const FLocKey& InNamespace, const FLocKey& InKey, const FLocItem& InSource, const FLocItem& InTranslation, TSharedPtr<FLocMetadataObject> InKeyMetadataObj, bool IsOptional)
	: Namespace(InNamespace)
	, Key(InKey)
	, Source(InSource)
	, Translation(InTranslation)
	, bIsOptional(IsOptional)
{
	if (InKeyMetadataObj.IsValid())
	{
		KeyMetadataObj = MakeShareable(new FLocMetadataObject(*InKeyMetadataObj));
	}
}

bool FInternationalizationArchive::AddEntry(const FLocKey& Namespace, const FLocKey& Key, const FLocItem& Source, const FLocItem& Translation, const TSharedPtr<FLocMetadataObject> KeyMetadataObj, const bool bOptional)
{
	if (Key.IsEmpty())
	{
		return false;
	}

	TSharedPtr<FArchiveEntry> ExistingEntry = FindEntryByKey(Namespace, Key, KeyMetadataObj);
	if (ExistingEntry.IsValid() && ExistingEntry->Source == Source)
	{
		return (ExistingEntry->Translation == Translation);
	}

	TSharedRef<FArchiveEntry> NewEntry = MakeShareable(new FArchiveEntry(Namespace, Key, Source, Translation, KeyMetadataObj, bOptional));
	if (ExistingEntry.IsValid())
	{
		UpdateEntry(ExistingEntry.ToSharedRef(), NewEntry);
	}
	else
	{
		EntriesBySourceText.Add(Source.Text, NewEntry);
		EntriesByKey.Add(Key, NewEntry);
	}

	return true;
}

bool FInternationalizationArchive::AddEntry(const TSharedRef<FArchiveEntry>& InEntry)
{
	return AddEntry(InEntry->Namespace, InEntry->Key, InEntry->Source, InEntry->Translation, InEntry->KeyMetadataObj, InEntry->bIsOptional);
}

void FInternationalizationArchive::UpdateEntry(const TSharedRef<FArchiveEntry>& OldEntry, const TSharedRef<FArchiveEntry>& NewEntry)
{
	EntriesBySourceText.RemoveSingle(OldEntry->Source.Text, OldEntry);
	EntriesBySourceText.Add(NewEntry->Source.Text, NewEntry);

	EntriesByKey.RemoveSingle(OldEntry->Key, OldEntry);
	EntriesByKey.Add(NewEntry->Key, NewEntry);
}

bool FInternationalizationArchive::SetTranslation(const FLocKey& Namespace, const FLocKey& Key, const FLocItem& Source, const FLocItem& Translation, const TSharedPtr<FLocMetadataObject> KeyMetadataObj)
{
	TSharedPtr<FArchiveEntry> Entry = FindEntryByKey(Namespace, Key, KeyMetadataObj);
	if (Entry.IsValid())
	{
		if (Entry->Source != Source)
		{
			UpdateEntry(Entry.ToSharedRef(), MakeShareable(new FArchiveEntry(Namespace, Key, Source, Translation, KeyMetadataObj, Entry->bIsOptional)));
		}
		else
		{
			Entry->Translation = Translation;
		}

		return true;
	}

	return false;
}

TSharedPtr<FArchiveEntry> FInternationalizationArchive::FindEntryByKey(const FLocKey& Namespace, const FLocKey& Key, const TSharedPtr<FLocMetadataObject> KeyMetadataObj) const
{
	TArray<TSharedRef<FArchiveEntry>, TInlineAllocator<4>> MatchingEntries;
	EntriesByKey.MultiFind(Key, MatchingEntries);

	for (const TSharedRef<FArchiveEntry>& Entry : MatchingEntries)
	{
		if (Entry->Namespace == Namespace)
		{
			if (!Entry->KeyMetadataObj.IsValid() && !KeyMetadataObj.IsValid())
			{
				return Entry;
			}
			else if ((KeyMetadataObj.IsValid() != Entry->KeyMetadataObj.IsValid()))
			{
				// If we are in here, we know that one of the metadata entries is null, if the other contains zero entries we will still consider them equivalent.
				if ((KeyMetadataObj.IsValid() && KeyMetadataObj->Values.Num() == 0) || (Entry->KeyMetadataObj.IsValid() && Entry->KeyMetadataObj->Values.Num() == 0))
				{
					return Entry;
				}
			}
			else if (*Entry->KeyMetadataObj == *KeyMetadataObj)
			{
				return Entry;
			}
		}
	}

	return nullptr;
}
