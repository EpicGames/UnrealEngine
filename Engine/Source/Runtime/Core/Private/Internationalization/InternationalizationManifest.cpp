// Copyright Epic Games, Inc. All Rights Reserved.

#include "Internationalization/InternationalizationManifest.h"

#include "Internationalization/InternationalizationMetadata.h"
#include "Logging/LogMacros.h"
#include "Misc/CString.h"

DEFINE_LOG_CATEGORY_STATIC(LogInternationalizationManifestObject, Log, All);

FManifestContext::FManifestContext(const FManifestContext& Other)
	: Key(Other.Key)
	, SourceLocation(Other.SourceLocation)
	, PlatformName(Other.PlatformName)
	, bIsOptional(Other.bIsOptional)
	, DevNotes(Other.DevNotes)
{
	if (Other.InfoMetadataObj.IsValid())
	{
		InfoMetadataObj = MakeShareable(new FLocMetadataObject(*Other.InfoMetadataObj));
	}

	if (Other.KeyMetadataObj.IsValid())
	{
		KeyMetadataObj = MakeShareable(new FLocMetadataObject(*Other.KeyMetadataObj));
	}
}

FManifestContext& FManifestContext::operator=(const FManifestContext& Other)
{
	if (this != &Other)
	{
		Key = Other.Key;
		SourceLocation = Other.SourceLocation;
		PlatformName = Other.PlatformName;
		bIsOptional = Other.bIsOptional;
		DevNotes = Other.DevNotes;
		InfoMetadataObj.Reset();
		KeyMetadataObj.Reset();
		if (Other.InfoMetadataObj.IsValid())
		{
			InfoMetadataObj = MakeShareable(new FLocMetadataObject(*(Other.InfoMetadataObj)));
		}

		if (Other.KeyMetadataObj.IsValid())
		{
			KeyMetadataObj = MakeShareable(new FLocMetadataObject(*(Other.KeyMetadataObj)));
		}
	}
	return *this;
}

bool FManifestContext::operator==(const FManifestContext& Other) const
{
	if (Key == Other.Key)
	{
		if (!KeyMetadataObj.IsValid() && !Other.KeyMetadataObj.IsValid())
		{
			return true;
		}
		else if (KeyMetadataObj.IsValid() != Other.KeyMetadataObj.IsValid())
		{
			// If we are in here, we know that one of the metadata entries is null, if the other contains zero entries we will still consider them equivalent.
			if ((KeyMetadataObj.IsValid() && KeyMetadataObj->Values.Num() == 0) || (Other.KeyMetadataObj.IsValid() && Other.KeyMetadataObj->Values.Num() == 0))
			{
				return true;
			}
			else
			{
				return false;
			}
		}
		return *(KeyMetadataObj) == *(Other.KeyMetadataObj);
	}
	return false;
}

bool FManifestContext::operator<(const FManifestContext& Other) const
{
	int32 Result = Key.Compare(Other.Key);

	bool bResult = false;
	if (Result == -1) //Is Less than
	{
		bResult = true;
	}
	else if (Result == 0) //Is Equal
	{
		if (KeyMetadataObj.IsValid() != Other.KeyMetadataObj.IsValid())
		{
			// If we are in here, we know that one of the metadata entries is null, if the other contains zero entries we will still consider them equivalent.
			if ((KeyMetadataObj.IsValid() && KeyMetadataObj->Values.Num() == 0) || (Other.KeyMetadataObj.IsValid() && Other.KeyMetadataObj->Values.Num() == 0))
			{
				return false;
			}
			else
			{
				bResult = Other.KeyMetadataObj.IsValid();
			}
		}
		else if (KeyMetadataObj.IsValid() && Other.KeyMetadataObj.IsValid())
		{
			bResult = (*(KeyMetadataObj) < *(Other.KeyMetadataObj));
		}
	}
	return bResult;
}

FLocItem::FLocItem(const FLocItem& Other)
	: Text(Other.Text)
{
	if (Other.MetadataObj.IsValid())
	{
		MetadataObj = MakeShareable(new FLocMetadataObject(*Other.MetadataObj));
	}
}

FLocItem& FLocItem::operator=(const FLocItem& Other)
{
	if (this != &Other)
	{
		Text = Other.Text;
		MetadataObj.Reset();

		if (Other.MetadataObj.IsValid())
		{
			MetadataObj = MakeShareable(new FLocMetadataObject(*Other.MetadataObj));
		}
	}
	return *this;
}

bool FLocItem::operator==(const FLocItem& Other) const
{
	if (Text.Equals(Other.Text, ESearchCase::CaseSensitive))
	{
		if (!MetadataObj.IsValid() && !Other.MetadataObj.IsValid())
		{
			return true;
		}
		else if (MetadataObj.IsValid() != Other.MetadataObj.IsValid())
		{
			// If we are in here, we know that one of the metadata entries is null, if the other contains zero entries we will still consider them equivalent.
			return ((MetadataObj.IsValid() && MetadataObj->Values.Num() == 0) || (Other.MetadataObj.IsValid() && Other.MetadataObj->Values.Num() == 0));
		}
		return *MetadataObj == *Other.MetadataObj;
	}
	return false;
}

bool FLocItem::operator<(const FLocItem& Other) const
{
	int32 Result = Text.Compare(Other.Text, ESearchCase::CaseSensitive);

	bool bResult = false;
	if (Result == -1) //Is less than
	{
		bResult = true;
	}
	else if (Result == 0) //Is equal
	{
		if (MetadataObj.IsValid() != Other.MetadataObj.IsValid())
		{
			// If we are in here, we know that one of the metadata entries is null, if the other contains zero entries we will still consider them equivalent.
			if ((MetadataObj.IsValid() && MetadataObj->Values.Num() == 0) || (Other.MetadataObj.IsValid() && Other.MetadataObj->Values.Num() == 0))
			{
				return false;
			}
			else
			{
				bResult = Other.MetadataObj.IsValid();
			}
		}
		else if (MetadataObj.IsValid() && Other.MetadataObj.IsValid())
		{
			bResult = (*MetadataObj < *Other.MetadataObj);
		}
	}
	return bResult;
}

bool FLocItem::IsExactMatch(const FLocItem& Other) const
{
	if (Text.Equals(Other.Text, ESearchCase::CaseSensitive))
	{
		return FLocMetadataObject::IsMetadataExactMatch(MetadataObj.Get(), Other.MetadataObj.Get());
	}
	return false;
}

bool FInternationalizationManifest::AddSource(const FLocKey& Namespace, const FLocItem& Source, const FManifestContext& Context)
{
	if (Context.Key.IsEmpty())
	{
		return false;
	}

	TSharedPtr<FManifestEntry> ExistingEntry = FindEntryByContext(Namespace, Context);

	if (ExistingEntry.IsValid())
	{
		return (Source.IsExactMatch(ExistingEntry->Source));
	}

	ExistingEntry = FindEntryBySource(Namespace, Source);

	if (ExistingEntry.IsValid() && !Source.IsExactMatch(ExistingEntry->Source))
	{
		return false;
	}

	if (!ExistingEntry.IsValid())
	{
		TSharedRef<FManifestEntry> NewEntry = MakeShareable(new FManifestEntry(Namespace, Source));
		EntriesBySourceText.Add(NewEntry->Source.Text, NewEntry);

		ExistingEntry = NewEntry;
	}

	ExistingEntry->Contexts.Add(Context);
	EntriesByKey.Add(Context.Key, ExistingEntry.ToSharedRef());
	return true;
}

void FInternationalizationManifest::UpdateEntry(const TSharedRef<FManifestEntry>& OldEntry, TSharedRef<FManifestEntry>& NewEntry)
{
	for (const FManifestContext& Context : OldEntry->Contexts)
	{
		EntriesByKey.RemoveSingle(Context.Key, OldEntry);
	}
	for (const FManifestContext& Context : NewEntry->Contexts)
	{
		EntriesByKey.Add(Context.Key, NewEntry);
	}

	EntriesBySourceText.RemoveSingle(OldEntry->Source.Text, OldEntry);
	EntriesBySourceText.Add(NewEntry->Source.Text, NewEntry);
}

TSharedPtr<FManifestEntry> FInternationalizationManifest::FindEntryBySource(const FLocKey& Namespace, const FLocItem& Source) const
{
	TArray<TSharedRef<FManifestEntry>, TInlineAllocator<4>> MatchingEntries;
	EntriesBySourceText.MultiFind(Source.Text, /*OUT*/MatchingEntries);

	for (const TSharedRef<FManifestEntry>& Entry : MatchingEntries)
	{
		if (Entry->Source == Source && Entry->Namespace == Namespace)
		{
			return Entry;
		}
	}

	return nullptr;
}

TSharedPtr<FManifestEntry> FInternationalizationManifest::FindEntryByContext(const FLocKey& Namespace, const FManifestContext& Context) const
{
	TArray<TSharedRef<FManifestEntry>, TInlineAllocator<4>> MatchingEntries;
	EntriesByKey.MultiFind(Context.Key, MatchingEntries);

	for (const TSharedRef<FManifestEntry>& Entry : MatchingEntries)
	{
		if (Entry->Namespace == Namespace)
		{
			for (const FManifestContext& EntryContext : Entry->Contexts)
			{
				if (EntryContext == Context)
				{
					return Entry;
				}
			}
		}
	}

	return nullptr;
}

TSharedPtr<FManifestEntry> FInternationalizationManifest::FindEntryByKey(const FLocKey& Namespace, const FLocKey& Key, const FString* SourceText) const
{
	TArray<TSharedRef<FManifestEntry>, TInlineAllocator<4>> MatchingEntries;
	EntriesByKey.MultiFind(Key, MatchingEntries);

	for (const TSharedRef<FManifestEntry>& MatchingEntry : MatchingEntries)
	{
		if (MatchingEntry->Namespace == Namespace && (!SourceText || MatchingEntry->Source.Text.Equals(*SourceText, ESearchCase::CaseSensitive)))
		{
			return MatchingEntry;
		}
	}

	return nullptr;
}

void FInternationalizationManifest::ClearEntries()
{
	EntriesBySourceText.Empty();
	EntriesByKey.Empty();
}

const FManifestContext* FManifestEntry::FindContext(const FLocKey& ContextKey, const TSharedPtr<FLocMetadataObject>& KeyMetadata) const
{
	for (const FManifestContext& Context : Contexts)
	{
		if (Context.Key == ContextKey)
		{
			if (Context.KeyMetadataObj.IsValid() != KeyMetadata.IsValid())
			{
				continue;
			}
			
			// If we get here, the meta-data objects are either both null, or both valid
			if (!Context.KeyMetadataObj.IsValid() || *Context.KeyMetadataObj == *KeyMetadata)
			{
				return &Context;
			}
		}
	}

	return nullptr;
}

const FManifestContext* FManifestEntry::FindContextByKey(const FLocKey& ContextKey) const
{
	for (const FManifestContext& Context : Contexts)
	{
		if (Context.Key == ContextKey)
		{
			return &Context;
		}
	}

	return nullptr;
}

void FManifestEntry::MergeContextPlatformInfo(const FManifestContext& InContext)
{
	for (FManifestContext& Context : Contexts)
	{
		if (Context.Key == InContext.Key)
		{
			if (Context.KeyMetadataObj.IsValid() != InContext.KeyMetadataObj.IsValid())
			{
				continue;
			}
			
			// If we get here, the meta-data objects are either both null, or both valid
			if (!Context.KeyMetadataObj.IsValid() || *Context.KeyMetadataObj == *InContext.KeyMetadataObj)
			{
				// If the platform name on this context doesn't match what we're being asked to merge to it, clear the 
				// platform name so that the text becomes platform agnostic (as it is being used by multiple platforms)
				// Also clear the source location to redact the info
				if (!Context.PlatformName.IsNone() && Context.PlatformName != InContext.PlatformName)
				{
					Context.PlatformName = FName();
					Context.SourceLocation.Reset();
				}
				if (InContext.PlatformName.IsNone() && Context.SourceLocation.IsEmpty())
				{
					// Previously redacted source location - use the agnostic source instead
					Context.SourceLocation = InContext.SourceLocation;
				}
			}
		}
	}
}
