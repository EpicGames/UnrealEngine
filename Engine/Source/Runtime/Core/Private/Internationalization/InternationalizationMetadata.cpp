// Copyright Epic Games, Inc. All Rights Reserved.

#include "Internationalization/InternationalizationMetadata.h"

#include "HAL/PlatformCrt.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Serialization/Archive.h"
#include "Serialization/StructuredArchive.h"
#include "Serialization/StructuredArchiveAdapters.h"
#include "Serialization/StructuredArchiveNameHelpers.h"
#include "Serialization/StructuredArchiveSlots.h"
#include "Templates/Less.h"
#include "Templates/Tuple.h"
#include "Trace/Detail/Channel.h"

DEFINE_LOG_CATEGORY_STATIC(LogInternationalizationMetadata, Log, All);

const TCHAR* FLocMetadataObject::COMPARISON_MODIFIER_PREFIX = TEXT("*");

FLocMetadataValue::~FLocMetadataValue() = default;

void FLocMetadataValue::ErrorMessage( const FString& InType )
{
	UE_LOGF(LogInternationalizationMetadata, Fatal, "LocMetadata Value of type '%ls' used as a '%ls'.", *GetTypeString(), *InType);
}

FLocMetadataObject::FLocMetadataObject( const FLocMetadataObject& Other )
{
	for( auto KeyIter = Other.Values.CreateConstIterator(); KeyIter; ++KeyIter )
	{
		const FString& Key = (*KeyIter).Key;
		const TSharedPtr< FLocMetadataValue > Value = (*KeyIter).Value;

		if( Value.IsValid() )
		{
			this->Values.Add( Key, Value->Clone() );
		}
	}
}

void FLocMetadataObject::SetField( const FString& FieldName, const TSharedPtr<FLocMetadataValue>& Value )
{
	this->Values.Add( FieldName, Value );
}

void FLocMetadataObject::RemoveField(const FString& FieldName)
{
	this->Values.Remove(FieldName);
}

FString FLocMetadataObject::GetStringField(const FString& FieldName)
{
	return GetField<ELocMetadataType::String>(FieldName)->AsString();
}

void FLocMetadataObject::SetStringField( const FString& FieldName, const FString& StringValue )
{
	this->Values.Add( FieldName, MakeShareable(new FLocMetadataValueString(StringValue)) );
}

bool FLocMetadataObject::GetBoolField(const FString& FieldName)
{
	return GetField<ELocMetadataType::Boolean>(FieldName)->AsBool();
}

void FLocMetadataObject::SetBoolField( const FString& FieldName, bool InValue )
{
	this->Values.Add( FieldName, MakeShareable( new FLocMetadataValueBoolean(InValue) ) );
}

TArray< TSharedPtr<FLocMetadataValue> > FLocMetadataObject::GetArrayField(const FString& FieldName)
{
	return GetField<ELocMetadataType::Array>(FieldName)->AsArray();
}

void FLocMetadataObject::SetArrayField( const FString& FieldName, const TArray< TSharedPtr<FLocMetadataValue> >& Array )
{
	this->Values.Add( FieldName, MakeShareable( new FLocMetadataValueArray(Array) ) );
}

TSharedPtr<FLocMetadataObject> FLocMetadataObject::GetObjectField(const FString& FieldName)
{
	return GetField<ELocMetadataType::Object>(FieldName)->AsObject();
}

void FLocMetadataObject::SetObjectField( const FString& FieldName, const TSharedPtr<FLocMetadataObject>& LocMetadataObject )
{
	if (LocMetadataObject.IsValid())
	{
		this->Values.Add( FieldName, MakeShareable(new FLocMetadataValueObject(LocMetadataObject.ToSharedRef())) );
	}
}

FLocMetadataObject& FLocMetadataObject::operator=( const FLocMetadataObject& Other )
{
	if( this != &Other )
	{
		this->Values.Empty( Other.Values.Num() );
		for( auto KeyIter = Other.Values.CreateConstIterator(); KeyIter; ++KeyIter )
		{
			const FString& Key = (*KeyIter).Key;
			const TSharedPtr< FLocMetadataValue > Value = (*KeyIter).Value;

			if( Value.IsValid() )
			{
				this->Values.Add( Key, Value->Clone() );
			}
		}
	}
	return *this;
}

bool FLocMetadataObject::operator==( const FLocMetadataObject& Other ) const
{
	if( Values.Num() != Other.Values.Num() )
	{
		return false;
	}

	for( auto KeyIter = Values.CreateConstIterator(); KeyIter; ++KeyIter )
	{
		const FString& Key = (*KeyIter).Key;
		const TSharedPtr< FLocMetadataValue > Value = (*KeyIter).Value;

		const TSharedPtr< FLocMetadataValue >* OtherValue = Other.Values.Find( Key );

		if( OtherValue && (*OtherValue).IsValid() )
		{
			// In the case where the name starts with the comparison modifier, we ignore the type and value.  Note, the contents of an array or object
			//  with this comparison modifier will not be checked even if those contents do not have the modifier.
			if( Key.StartsWith( COMPARISON_MODIFIER_PREFIX ) )
			{
				continue;
			}
			else if( Value->GetType() == (*OtherValue)->GetType() && *(Value) == *(*OtherValue) )
			{
				continue;
			}
			else
			{
				return false;
			}
		}
	}
	return true;
}


bool FLocMetadataObject::IsExactMatch( const FLocMetadataObject& Other ) const
{
	if( Values.Num() != Other.Values.Num() )
	{
		return false;
	}

	for( auto KeyIter = Values.CreateConstIterator(); KeyIter; ++KeyIter )
	{
		const FString& Key = (*KeyIter).Key;
		const TSharedPtr< FLocMetadataValue > Value = (*KeyIter).Value;

		const TSharedPtr< FLocMetadataValue >* OtherValue = Other.Values.Find( Key );

		if( OtherValue && (*OtherValue).IsValid() )
		{
			if( Value->GetType() == (*OtherValue)->GetType() && *(Value) == *(*OtherValue) )
			{
				continue;
			}
			else
			{
				return false;
			}
		}
	}
	return true;
}

bool FLocMetadataObject::operator<( const FLocMetadataObject& Other ) const
{
	TArray< FString > MetaKeys;
	Values.GetKeys( MetaKeys );

	TArray< FString > OtherMetaKeys;
	Other.Values.GetKeys( OtherMetaKeys );

	MetaKeys.Sort( TLess<FString>() );
	OtherMetaKeys.Sort( TLess<FString>() );

	for( int32 MetaKeyIdx = 0; MetaKeyIdx < MetaKeys.Num(); ++MetaKeyIdx )
	{
		if( MetaKeyIdx >= OtherMetaKeys.Num() )
		{
			// We are not less than the other we are comparing list because it ran out of metadata keys while we still have some
			return false;
		}
		else if ( MetaKeys[ MetaKeyIdx ] != OtherMetaKeys[ MetaKeyIdx ] )
		{
			return MetaKeys[ MetaKeyIdx ] < OtherMetaKeys[ MetaKeyIdx ];
		}
	}

	if( OtherMetaKeys.Num() > MetaKeys.Num() )
	{
		// All the keys are the same up to this point but other has additional keys
		return true;
	}

	// If we get here, we know that all the keys are the same so now we start comparing values.
	for( int32 MetaKeyIdx = 0; MetaKeyIdx < MetaKeys.Num(); ++MetaKeyIdx )
	{
		const TSharedPtr< FLocMetadataValue >* Value = Values.Find( MetaKeys[ MetaKeyIdx ] );
		const TSharedPtr< FLocMetadataValue >* OtherValue = Other.Values.Find( MetaKeys[ MetaKeyIdx ] );

		if( !Value && !OtherValue )
		{
			continue;
		}
		else if( (Value == NULL) != (OtherValue == NULL) )
		{
			return (OtherValue == NULL);
		}

		if( !(*Value).IsValid() && !(*OtherValue).IsValid() )
		{
			continue;
		}
		else if( (*Value).IsValid() != (*OtherValue).IsValid() )
		{
			return (*OtherValue).IsValid();
		}
		else if(*(*Value) < *(*OtherValue))
		{
			 return true;
		}
		else if(!(*(*Value) == *(*OtherValue)) )
		{
			return false;
		}

	}
	return false;
}

bool FLocMetadataObject::IsMetadataExactMatch( const FLocMetadataObject* const MetadataA, const FLocMetadataObject* const MetadataB )
{
	if( !MetadataA && !MetadataB )
	{
		return true;
	}
	else if( (MetadataA != nullptr) != (MetadataB != nullptr) )
	{
		// If we are in here, we know that one of the metadata entries is null, if the other
		//  contains zero entries we will still consider them equivalent.
		if( (MetadataA && MetadataA->Values.Num() == 0) ||
			(MetadataB && MetadataB->Values.Num() == 0) )
		{
			return true;
		}
		else
		{
			return false;
		}
	}
	// Note: Since the standard source comparison operator handles * prefixed meta data in a special way, we use an exact match
	//  check here instead.
	return MetadataA->IsExactMatch( *(MetadataB) );
}

FString FLocMetadataObject::ToString() const
{
	FString MemberList;
	for (const auto& Pair : Values)
	{
		const FString MemberName = Pair.Key;
		const TSharedPtr<FLocMetadataValue> MemberValue = Pair.Value;
		MemberList += (MemberList.IsEmpty() ? TEXT("") : TEXT(",")) + MemberValue->ToString();
	}
	return FString::Printf(TEXT("{%s}"), *MemberList);
}

namespace
{
	void SerializeLocMetadataValue(FStructuredArchive::FSlot Slot, TSharedPtr<FLocMetadataValue>& Value)
	{
		FStructuredArchive::FRecord Record = Slot.EnterRecord();
		FArchive& BaseArchive = Slot.GetUnderlyingArchive();

		bool bIsLoading = BaseArchive.IsLoading();

		if (bIsLoading)
		{
			check(!Value.IsValid());
		}
		else
		{
			check(Value.IsValid());
		}

		ELocMetadataType MetaDataType = ELocMetadataType::None;
		if (!bIsLoading)
		{
			MetaDataType = Value->GetType();
		}

		int32 MetaDataTypeAsInt = static_cast<int32>(MetaDataType);
		Record << SA_VALUE(TEXT("Type"), MetaDataTypeAsInt);
		MetaDataType = static_cast<ELocMetadataType>(MetaDataTypeAsInt);

		FStructuredArchive::FSlot ValueSlot = Record.EnterField(TEXT("Value"));

		switch (MetaDataType)
		{
		case ELocMetadataType::Array:
			if (bIsLoading)
			{
				Value = MakeShareable(new FLocMetadataValueArray(ValueSlot));
			}
			else
			{
				FLocMetadataValueArray::Serialize(static_cast<FLocMetadataValueArray&>(*Value), ValueSlot);
			}
			break;
		case ELocMetadataType::Boolean:
			if (bIsLoading)
			{
				Value = MakeShareable(new FLocMetadataValueBoolean(ValueSlot));
			}
			else
			{
				FLocMetadataValueBoolean::Serialize(static_cast<FLocMetadataValueBoolean&>(*Value), ValueSlot);
			}
			break;
		case ELocMetadataType::Object:
			if (bIsLoading)
			{
				Value = MakeShareable(new FLocMetadataValueObject(ValueSlot));
			}
			else
			{
				FLocMetadataValueObject::Serialize(static_cast<FLocMetadataValueObject&>(*Value), ValueSlot);
			}
			break;
		case ELocMetadataType::String:
			if (bIsLoading)
			{
				Value = MakeShareable(new FLocMetadataValueString(ValueSlot));
			}
			else
			{
				FLocMetadataValueString::Serialize(static_cast<FLocMetadataValueString&>(*Value), ValueSlot);
			}
			break;
		default:
			checkNoEntry();
		}
	}
}

void operator<<(FStructuredArchive::FSlot Slot, FLocMetadataObject& Object)
{
	FStructuredArchive::FRecord Record = Slot.EnterRecord();
	int32 ValueCount = Object.Values.Num();
	Record << SA_VALUE(TEXT("ValueCount"), ValueCount);

	bool bIsLoading = Slot.GetArchiveState().IsLoading();

	if (bIsLoading)
	{
		Object.Values.Reserve(ValueCount);
	}

	FStructuredArchive::FStream Stream = Record.EnterStream(TEXT("Values"));
	TArray<FString> MapKeys;
	Object.Values.GetKeys(MapKeys);
	for (int32 i = 0; i < ValueCount; ++i)
	{
		FStructuredArchive::FRecord ValueRecord = Stream.EnterElement().EnterRecord();

		FString Key;

		if (!bIsLoading)
		{
			Key = MapKeys[i];
		}

		ValueRecord << SA_VALUE(TEXT("Key"), Key);

		FStructuredArchive::FSlot ValueSlot = ValueRecord.EnterField(TEXT("Value"));
		if (bIsLoading)
		{
			TSharedPtr<FLocMetadataValue> Value;
			SerializeLocMetadataValue(ValueSlot, Value);
			Object.Values.Add(Key, Value);
		}
		else
		{
			SerializeLocMetadataValue(ValueSlot, Object.Values[Key]);
		}
	}
}

FArchive& operator<<(FArchive& Archive, FLocMetadataObject& Object)
{
	FStructuredArchiveFromArchive(Archive).GetSlot() << Object;
	return Archive;
}

bool FLocMetadataValueString::EqualTo( const FLocMetadataValue& Other ) const
{
	const FLocMetadataValueString* OtherObj = (FLocMetadataValueString*) &Other;
	return (Value == OtherObj->Value);
}

bool FLocMetadataValueString::LessThan( const FLocMetadataValue& Other ) const
{
	const FLocMetadataValueString* OtherObj = (FLocMetadataValueString*) &Other;
	return (Value < OtherObj->Value);
}

TSharedRef<FLocMetadataValue> FLocMetadataValueString::Clone() const
{
	return MakeShareable( new FLocMetadataValueString( Value ) );
}

FLocMetadataValueString::FLocMetadataValueString(FStructuredArchive::FSlot Slot)
{
	check(Slot.GetArchiveState().IsLoading());
	Slot << Value;
}

void FLocMetadataValueString::Serialize( FLocMetadataValueString& Value, FStructuredArchive::FSlot Slot )
{
	check(!Slot.GetArchiveState().IsLoading());

	FString StringValue = Value.Value;
	Slot << StringValue;
}

bool FLocMetadataValueBoolean::EqualTo( const FLocMetadataValue& Other ) const
{
	const FLocMetadataValueBoolean* OtherObj = (FLocMetadataValueBoolean*) &Other;
	return (Value == OtherObj->Value);
}

bool FLocMetadataValueBoolean::LessThan( const FLocMetadataValue& Other ) const
{
	const FLocMetadataValueBoolean* OtherObj = (FLocMetadataValueBoolean*) &Other;
	return (Value != OtherObj->Value) ? (!Value) : false;
}

TSharedRef<FLocMetadataValue> FLocMetadataValueBoolean::Clone() const
{
	return MakeShareable( new FLocMetadataValueBoolean( Value ) );
}

FLocMetadataValueBoolean::FLocMetadataValueBoolean(FStructuredArchive::FSlot Slot)
{
	check(Slot.GetArchiveState().IsLoading());

	Slot << Value;
}

void FLocMetadataValueBoolean::Serialize( FLocMetadataValueBoolean& Value, FStructuredArchive::FSlot Slot )
{
	check(!Slot.GetArchiveState().IsLoading());

	bool BoolValue = Value.Value;
	Slot << BoolValue;
}

struct FCompareMetadataValue
{
	FORCEINLINE bool operator()( TSharedPtr<FLocMetadataValue> A, TSharedPtr<FLocMetadataValue> B ) const
	{
		if( !A.IsValid() && !B.IsValid() )
		{
			return false;
		}
		else if( A.IsValid() != B.IsValid() )
		{
			return B.IsValid();
		}
		return *A < *B;
	}
};

bool FLocMetadataValueArray::EqualTo( const FLocMetadataValue& Other ) const
{
	const FLocMetadataValueArray* OtherObj = (FLocMetadataValueArray*) &Other; 

	if( Value.Num() != OtherObj->Value.Num() )
	{
		return false;
	}

	TArray< TSharedPtr<FLocMetadataValue> > Sorted = Value;
	TArray< TSharedPtr<FLocMetadataValue> > OtherSorted = OtherObj->Value;

	Sorted.Sort( FCompareMetadataValue() );
	OtherSorted.Sort( FCompareMetadataValue() );

	for( int32 idx = 0; idx < Sorted.Num(); ++idx )
	{
		if( !(*(Sorted[idx]) == *(OtherSorted[idx]) ) )
		{
			return false;
		}
	}

	return true;
}

bool FLocMetadataValueArray::LessThan( const FLocMetadataValue& Other ) const
{
	const FLocMetadataValueArray* OtherObj = (FLocMetadataValueArray*) &Other;

	TArray< TSharedPtr<FLocMetadataValue> > Sorted = Value;
	TArray< TSharedPtr<FLocMetadataValue> > OtherSorted = OtherObj->Value;

	Sorted.Sort( FCompareMetadataValue() );
	OtherSorted.Sort( FCompareMetadataValue() );

	for( int32 idx = 0; idx < Sorted.Num(); ++idx )
	{
		if( idx >= OtherSorted.Num() )
		{
			return false;
		}
		else if( !( *(Sorted[idx]) == *(OtherSorted[idx]) ) )
		{
			return *(Sorted[idx]) < *(OtherSorted[idx]);
		}
	}

	if( OtherSorted.Num() > Sorted.Num() )
	{
		return true;
	}

	return false;
}

TSharedRef<FLocMetadataValue> FLocMetadataValueArray::Clone() const
{
	TArray< TSharedPtr<FLocMetadataValue> > NewValue;
	for( auto Iter = Value.CreateConstIterator(); Iter; ++Iter )
	{
		NewValue.Add( (*Iter)->Clone() );
	}
	return MakeShareable( new FLocMetadataValueArray( NewValue ) );
}

FString FLocMetadataValueArray::ToString() const
{
	FString ElementList;
	for (const TSharedPtr<FLocMetadataValue>& Element : Value)
	{
		ElementList += (ElementList.IsEmpty() ? TEXT("") : TEXT(",")) + Element->ToString();
	}
	return FString::Printf(TEXT("[%s]"), *ElementList);
}

FLocMetadataValueArray::FLocMetadataValueArray(FStructuredArchive::FSlot Slot)
{
	check(Slot.GetArchiveState().IsLoading());

	int32 ElementCount;
	FStructuredArchive::FArray Array = Slot.EnterArray(ElementCount);
	Value.SetNum(ElementCount);

	for (TSharedPtr<FLocMetadataValue>& Element : Value)
	{
		SerializeLocMetadataValue(Array.EnterElement(), Element);
	}
}

void FLocMetadataValueArray::Serialize(FLocMetadataValueArray& Value, FStructuredArchive::FSlot Slot)
{
	check(!Slot.GetArchiveState().IsLoading());

	int32 ElementCount = Value.Value.Num();
	FStructuredArchive::FArray Array = Slot.EnterArray(ElementCount);

	for (TSharedPtr<FLocMetadataValue>& Element : Value.Value)
	{
		SerializeLocMetadataValue(Array.EnterElement(), Element);
	}
}

bool FLocMetadataValueObject::EqualTo( const FLocMetadataValue& Other ) const
{
	const FLocMetadataValueObject* OtherObj = (FLocMetadataValueObject*) &Other;  
	if( !Value.IsValid() && !OtherObj->Value.IsValid() )
	{
		return true;
	}
	else if( Value.IsValid() != OtherObj->Value.IsValid() )
	{
		return false;
	}
	return *(Value) == *(OtherObj->Value);
}

bool FLocMetadataValueObject::LessThan( const FLocMetadataValue& Other ) const
{
	const FLocMetadataValueObject* OtherObj = (FLocMetadataValueObject*) &Other;
	if( !Value.IsValid() && !OtherObj->Value.IsValid() )
	{
		return false;
	}
	else if( Value.IsValid() != OtherObj->Value.IsValid() )
	{
		return OtherObj->Value.IsValid();
	}
	return *(Value) < *(OtherObj->Value);
}

TSharedRef<FLocMetadataValue> FLocMetadataValueObject::Clone() const
{
	TSharedRef<FLocMetadataObject> NewLocMetadataObject = MakeShareable( new FLocMetadataObject( *(this->Value) ) );
	return MakeShareable( new FLocMetadataValueObject( NewLocMetadataObject ) );
}

FString FLocMetadataValueObject::ToString() const
{
	return Value->ToString();
}

FLocMetadataValueObject::FLocMetadataValueObject(FStructuredArchive::FSlot Slot)
	: Value(new FLocMetadataObject)
{
	check(Slot.GetArchiveState().IsLoading());

	Slot << *Value;
}

void FLocMetadataValueObject::Serialize( FLocMetadataValueObject& Value, FStructuredArchive::FSlot Slot )
{
	check(!Slot.GetArchiveState().IsLoading());

	Slot << *(Value.Value);
}
