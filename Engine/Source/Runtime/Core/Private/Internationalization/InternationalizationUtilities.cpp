// Copyright Epic Games, Inc. All Rights Reserved.

#include "Internationalization/InternationalizationUtilities.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Cultures/LeetCulture.h"
#include "Internationalization/Cultures/KeysCulture.h"

namespace InternationalizationUtilities
{
	FString SanitizeCultureCode(const FString& InCultureCode)
	{
		if (InCultureCode.IsEmpty())
		{
			return InCultureCode;
		}

		// UE culture codes (IETF language tags) may only contain A-Z, a-z, 0-9, -, ,_, @, ;, =, or .
		FString SanitizedCultureCode = InCultureCode;
		{
			SanitizedCultureCode.GetCharArray().RemoveAll([](const TCHAR InChar)
			{
				if (InChar != 0)
				{
					const bool bIsValid = (InChar >= TEXT('A') && InChar <= TEXT('Z')) || (InChar >= TEXT('a') && InChar <= TEXT('z')) || (InChar >= TEXT('0') && InChar <= TEXT('9')) || (InChar == TEXT('-')) || (InChar == TEXT('_')) || (InChar == TEXT('@')) || (InChar == TEXT(';')) || (InChar == TEXT('=')) || (InChar == TEXT('.'));
					return !bIsValid;
				}
				return false;
			});
		}
		return SanitizedCultureCode;
	}

	FString SanitizeTimezoneCode(const FString& InTimezoneCode)
	{
		if (InTimezoneCode.IsEmpty())
		{
			return InTimezoneCode;
		}

		// UE timezone codes (Olson or custom offset codes) may only contain A-Z, a-z, 0-9, :, /, +, -, or _, and each / delimited name can be 14-characters max
		FString SanitizedTimezoneCode = InTimezoneCode;
		{
			int32 NumValidChars = 0;
			SanitizedTimezoneCode.GetCharArray().RemoveAll([&NumValidChars](const TCHAR InChar)
			{
				if (InChar != 0)
				{
					if (InChar == TEXT('/'))
					{
						NumValidChars = 0;
						return false;
					}
					else
					{
						const bool bIsValid = (InChar >= TEXT('A') && InChar <= TEXT('Z')) || (InChar >= TEXT('a') && InChar <= TEXT('z')) || (InChar >= TEXT('0') && InChar <= TEXT('9')) || (InChar == TEXT(':')) || (InChar == TEXT('+')) || (InChar == TEXT('-')) || (InChar == TEXT('_'));
						return !bIsValid || ++NumValidChars > 14;
					}
				}
				return false;
			});
		}
		return SanitizedTimezoneCode;
	}

	FString SanitizeCurrencyCode(const FString& InCurrencyCode)
	{
		if (InCurrencyCode.IsEmpty())
		{
			return InCurrencyCode;
		}

		// UE currency codes (ISO 4217) may only contain A-Z or a-z, and should be 3-characters
		FString SanitizedCurrencyCode = InCurrencyCode;
		{
			int32 NumValidChars = 0;
			SanitizedCurrencyCode.GetCharArray().RemoveAll([&NumValidChars](const TCHAR InChar)
			{
				if (InChar != 0)
				{
					const bool bIsValid = IsValidCurencyCodeCharacter(InChar);
					return !bIsValid || ++NumValidChars > 3;
				}
				return false;
			});
		}
		return SanitizedCurrencyCode;
	}

	FString GetCanonicalCultureName(const FString& Name, const FString& FallbackCulture, FInternationalization& I18N)
	{
		auto IsLanguageCode = [](const FString& InCode)
		{
			// Language codes must be 2 or 3 letters, or our special "LEET"/"keys" languages
			return InCode.Len() == 2
				|| InCode.Len() == 3
#if ENABLE_LOC_TESTING
				|| InCode == FLeetCulture::StaticGetName()
				|| InCode == FKeysCulture::StaticGetName()
#endif
				;
		};

		auto IsScriptCode = [](const FString& InCode)
		{
			// Script codes must be 4 letters
			return InCode.Len() == 4;
		};

		auto IsRegionCode = [](const FString& InCode)
		{
			// Region codes must be 2 or 3 letters
			return InCode.Len() == 2 || InCode.Len() == 3;
		};

		auto ConditionLanguageCode = [](FString& InOutCode)
		{
			// Language codes are lowercase
			InOutCode.ToLowerInline();
		};

		auto ConditionScriptCode = [](FString& InOutCode)
		{
			// Script codes are titlecase
			InOutCode.ToLowerInline();
			if (InOutCode.Len() > 0)
			{
				InOutCode[0] = FChar::ToUpper(InOutCode[0]);
			}
		};

		auto ConditionRegionCode = [](FString& InOutCode)
		{
			// Region codes are uppercase
			InOutCode.ToUpperInline();
		};

		auto ConditionVariant = [](FString& InOutVariant)
		{
			// Variants are uppercase
			InOutVariant.ToUpperInline();
		};

		auto ConditionKeywordArgKey = [](FString& InOutKey)
		{
			static const FString ValidKeywords[] = {
				TEXT("calendar"),
				TEXT("collation"),
				TEXT("currency"),
				TEXT("numbers"),
			};

			// Keyword argument keys are lowercase
			InOutKey.ToLowerInline();

			// Only certain argument keys are accepted
			for (const FString& ValidKeyword : ValidKeywords)
			{
				if (InOutKey.Equals(ValidKeyword, ESearchCase::CaseSensitive))
				{
					return;
				}
			}

			// Invalid key - clear it
			InOutKey.Reset();
		};

		enum class ENameTagType : uint8
		{
			Language,
			Script,
			Region,
			Variant,
		};

		struct FNameTag
		{
			FString Str;
			ENameTagType Type;
		};

		struct FCanonizedTagData
		{
			const TCHAR* CanonizedNameTag;
			const TCHAR* KeywordArgKey;
			const TCHAR* KeywordArgValue;
		};

		static const TSortedMap<FString, FCanonizedTagData> CanonizedTagMap = []()
		{
			TSortedMap<FString, FCanonizedTagData> TmpCanonizedTagMap;
			TmpCanonizedTagMap.Add(TEXT(""), { TEXT("en-US-POSIX"), nullptr, nullptr });
			TmpCanonizedTagMap.Add(TEXT("c"), { TEXT("en-US-POSIX"), nullptr, nullptr });
			TmpCanonizedTagMap.Add(TEXT("posix"), { TEXT("en-US-POSIX"), nullptr, nullptr });
			TmpCanonizedTagMap.Add(TEXT("ca-ES-PREEURO"), { TEXT("ca-ES"), TEXT("currency"), TEXT("ESP") });
			TmpCanonizedTagMap.Add(TEXT("de-AT-PREEURO"), { TEXT("de-AT"), TEXT("currency"), TEXT("ATS") });
			TmpCanonizedTagMap.Add(TEXT("de-DE-PREEURO"), { TEXT("de-DE"), TEXT("currency"), TEXT("DEM") });
			TmpCanonizedTagMap.Add(TEXT("de-LU-PREEURO"), { TEXT("de-LU"), TEXT("currency"), TEXT("LUF") });
			TmpCanonizedTagMap.Add(TEXT("el-GR-PREEURO"), { TEXT("el-GR"), TEXT("currency"), TEXT("GRD") });
			TmpCanonizedTagMap.Add(TEXT("en-BE-PREEURO"), { TEXT("en-BE"), TEXT("currency"), TEXT("BEF") });
			TmpCanonizedTagMap.Add(TEXT("en-IE-PREEURO"), { TEXT("en-IE"), TEXT("currency"), TEXT("IEP") });
			TmpCanonizedTagMap.Add(TEXT("es-ES-PREEURO"), { TEXT("es-ES"), TEXT("currency"), TEXT("ESP") });
			TmpCanonizedTagMap.Add(TEXT("eu-ES-PREEURO"), { TEXT("eu-ES"), TEXT("currency"), TEXT("ESP") });
			TmpCanonizedTagMap.Add(TEXT("fi-FI-PREEURO"), { TEXT("fi-FI"), TEXT("currency"), TEXT("FIM") });
			TmpCanonizedTagMap.Add(TEXT("fr-BE-PREEURO"), { TEXT("fr-BE"), TEXT("currency"), TEXT("BEF") });
			TmpCanonizedTagMap.Add(TEXT("fr-FR-PREEURO"), { TEXT("fr-FR"), TEXT("currency"), TEXT("FRF") });
			TmpCanonizedTagMap.Add(TEXT("fr-LU-PREEURO"), { TEXT("fr-LU"), TEXT("currency"), TEXT("LUF") });
			TmpCanonizedTagMap.Add(TEXT("ga-IE-PREEURO"), { TEXT("ga-IE"), TEXT("currency"), TEXT("IEP") });
			TmpCanonizedTagMap.Add(TEXT("gl-ES-PREEURO"), { TEXT("gl-ES"), TEXT("currency"), TEXT("ESP") });
			TmpCanonizedTagMap.Add(TEXT("it-IT-PREEURO"), { TEXT("it-IT"), TEXT("currency"), TEXT("ITL") });
			TmpCanonizedTagMap.Add(TEXT("nl-BE-PREEURO"), { TEXT("nl-BE"), TEXT("currency"), TEXT("BEF") });
			TmpCanonizedTagMap.Add(TEXT("nl-NL-PREEURO"), { TEXT("nl-NL"), TEXT("currency"), TEXT("NLG") });
			TmpCanonizedTagMap.Add(TEXT("pt-PT-PREEURO"), { TEXT("pt-PT"), TEXT("currency"), TEXT("PTE") });
			return TmpCanonizedTagMap;
		}();

		static const TSortedMap<FString, FCanonizedTagData> VariantMap = []()
		{
			TSortedMap<FString, FCanonizedTagData> TmpVariantMap;
			TmpVariantMap.Add(TEXT("EURO"), { nullptr, TEXT("currency"), TEXT("EUR") });
			return TmpVariantMap;
		}();

		// Sanitize any nastiness from the culture code
		const FString SanitizedName = SanitizeCultureCode(Name);

		// If the name matches a custom culture, then just accept it as-is
		if (I18N.GetCustomCulture(SanitizedName))
		{
			return SanitizedName;
		}

		// These will be populated as the string is processed and are used to re-build the canonized string
		TArray<FNameTag, TInlineAllocator<4>> ParsedNameTags;
		TSortedMap<FString, FString, TInlineAllocator<4>> ParsedKeywords;

		// Parse the string into its component parts
		{
			// 1) Split the string so that the keywords exist in a separate string (both halves need separate processing)
			FString NameTag;
			FString NameKeywords;
			{
				int32 NameKeywordsSplitIndex = INDEX_NONE;
				SanitizedName.FindChar(TEXT('@'), NameKeywordsSplitIndex);

				int32 EncodingSplitIndex = INDEX_NONE;
				SanitizedName.FindChar(TEXT('.'), EncodingSplitIndex);

				// The name tags part of the string ends at either the start of the keywords or encoding (whichever is smaller)
				const int32 NameTagEndIndex = FMath::Min(
					NameKeywordsSplitIndex == INDEX_NONE ? SanitizedName.Len() : NameKeywordsSplitIndex,
					EncodingSplitIndex == INDEX_NONE ? SanitizedName.Len() : EncodingSplitIndex
				);

				NameTag = SanitizedName.Left(NameTagEndIndex);
				NameTag.ReplaceCharInline(TEXT('_'), TEXT('-'), ESearchCase::CaseSensitive);

				if (NameKeywordsSplitIndex != INDEX_NONE)
				{
					NameKeywords = SanitizedName.Mid(NameKeywordsSplitIndex + 1);
				}
			}

			// 2) Perform any wholesale substitution (which may also add keywords into ParsedKeywords)
			if (const FCanonizedTagData* CanonizedTagData = CanonizedTagMap.Find(NameTag))
			{
				NameTag = CanonizedTagData->CanonizedNameTag;
				if (CanonizedTagData->KeywordArgKey && CanonizedTagData->KeywordArgValue)
				{
					ParsedKeywords.Add(CanonizedTagData->KeywordArgKey, CanonizedTagData->KeywordArgValue);
				}
			}

			// 3) Split the name tag into its component parts (produces the initial set of ParsedNameTags)
			{
				int32 NameTagStartIndex = 0;
				int32 NameTagEndIndex = 0;
				do
				{
					// Walk to the next breaking point
					for (; NameTagEndIndex < NameTag.Len() && NameTag[NameTagEndIndex] != TEXT('-'); ++NameTagEndIndex) {}

					// Process the tag
					{
						FString NameTagStr = NameTag.Mid(NameTagStartIndex, NameTagEndIndex - NameTagStartIndex);
						const FCanonizedTagData* VariantTagData = nullptr;

						// What kind of tag is this?
						ENameTagType NameTagType = ENameTagType::Variant;
						if (ParsedNameTags.Num() == 0 && IsLanguageCode(NameTagStr))
						{
							// todo: map 3 letter language codes into 2 letter language codes like ICU would?
							NameTagType = ENameTagType::Language;
							ConditionLanguageCode(NameTagStr);
						}
						else if (ParsedNameTags.Num() == 1 && ParsedNameTags.Last().Type == ENameTagType::Language && IsScriptCode(NameTagStr))
						{
							NameTagType = ENameTagType::Script;
							ConditionScriptCode(NameTagStr);
						}
						else if (ParsedNameTags.Num() > 0 && ParsedNameTags.Num() <= 2 && (ParsedNameTags.Last().Type == ENameTagType::Language || ParsedNameTags.Last().Type == ENameTagType::Script) && IsRegionCode(NameTagStr))
						{
							// todo: map 3 letter region codes into 2 letter region codes like ICU would?
							NameTagType = ENameTagType::Region;
							ConditionRegionCode(NameTagStr);
						}
						else
						{
							ConditionVariant(NameTagStr);
							VariantTagData = VariantMap.Find(NameTagStr);
						}

						if (VariantTagData)
						{
							check(VariantTagData->KeywordArgKey && VariantTagData->KeywordArgValue);
							ParsedKeywords.Add(VariantTagData->KeywordArgKey, VariantTagData->KeywordArgValue);
						}
						else if (NameTagStr.Len() > 0)
						{
							ParsedNameTags.Add({ MoveTemp(NameTagStr), NameTagType });
						}
					}

					// Prepare for the next loop
					NameTagStartIndex = NameTagEndIndex + 1;
					NameTagEndIndex = NameTagStartIndex;
				} while (NameTagEndIndex < NameTag.Len());
			}

			// 4) Parse the keywords (this may produce both variants into ParsedNameTags, and keywords into ParsedKeywords)
			{
				TArray<FString> NameKeywordArgs;
				NameKeywords.ParseIntoArray(NameKeywordArgs, TEXT(";"));

				for (FString& NameKeywordArg : NameKeywordArgs)
				{
					int32 KeyValueSplitIndex = INDEX_NONE;
					NameKeywordArg.FindChar(TEXT('='), KeyValueSplitIndex);

					if (KeyValueSplitIndex == INDEX_NONE)
					{
						// Single values are treated as variants
						ConditionVariant(NameKeywordArg);
						if (NameKeywordArg.Len() > 0)
						{
							ParsedNameTags.Add({ MoveTemp(NameKeywordArg), ENameTagType::Variant });
						}
					}
					else
					{
						// Key->Value pairs are treated as keywords
						FString NameKeywordArgKey = NameKeywordArg.Left(KeyValueSplitIndex);
						ConditionKeywordArgKey(NameKeywordArgKey);
						FString NameKeywordArgValue = NameKeywordArg.Mid(KeyValueSplitIndex + 1);
						if (NameKeywordArgKey.Len() > 0 && NameKeywordArgValue.Len() > 0)
						{
							if (!ParsedKeywords.Contains(NameKeywordArgKey))
							{
								ParsedKeywords.Add(MoveTemp(NameKeywordArgKey), MoveTemp(NameKeywordArgValue));
							}
						}
					}
				}
			}
		}

		// Re-assemble the string into its canonized form
		FString CanonicalName;
		{
			// Assemble the name tags first
			// These *must* start with a language tag
			if (ParsedNameTags.Num() > 0 && ParsedNameTags[0].Type == ENameTagType::Language)
			{
				for (int32 NameTagIndex = 0; NameTagIndex < ParsedNameTags.Num(); ++NameTagIndex)
				{
					const FNameTag& NameTag = ParsedNameTags[NameTagIndex];

					switch (NameTag.Type)
					{
					case ENameTagType::Language:
						CanonicalName = NameTag.Str;
						break;

					case ENameTagType::Script:
					case ENameTagType::Region:
						CanonicalName += TEXT('-');
						CanonicalName += NameTag.Str;
						break;

					case ENameTagType::Variant:
						// If the previous tag was a language, we need to add an extra hyphen for non-empty variants since ICU would produce a double hyphen in this case
						if (ParsedNameTags.IsValidIndex(NameTagIndex - 1) && ParsedNameTags[NameTagIndex - 1].Type == ENameTagType::Language && !NameTag.Str.IsEmpty())
						{
							CanonicalName += TEXT('-');
						}
						CanonicalName += TEXT('-');
						CanonicalName += NameTag.Str;
						break;

					default:
						break;
					}
				}
			}

			// Now add the keywords
			if (CanonicalName.Len() > 0 && ParsedKeywords.Num() > 0)
			{
				TCHAR NextToken = TEXT('@');
				for (const auto& ParsedKeywordPair : ParsedKeywords)
				{
					CanonicalName += NextToken;
					NextToken = TEXT(';');

					CanonicalName += ParsedKeywordPair.Key;
					CanonicalName += TEXT('=');
					CanonicalName += ParsedKeywordPair.Value;
				}
			}

			// If we canonicalized to an empty string, use the given fallback
			if (CanonicalName.IsEmpty())
			{
				CanonicalName = FallbackCulture;
			}
		}
		return CanonicalName;
	}
}
