// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/UnrealString.h"

class FInternationalization;

namespace InternationalizationUtilities
{
	/** Sanitize the given culture code so that it is safe to use */
	FString SanitizeCultureCode(const FString& InCultureCode);

	/** Sanitize the given timezone code so that it is safe to use */
	FString SanitizeTimezoneCode(const FString& InTimezoneCode);

	/** Sanitize the given currency code so that it is safe to use */
	FString SanitizeCurrencyCode(const FString& InCurrencyCode);

	/** Is the given character valid as part of currency code? */
	inline bool IsValidCurencyCodeCharacter(const TCHAR InChar)
	{
		return (InChar >= TEXT('A') && InChar <= TEXT('Z')) || (InChar >= TEXT('a') && InChar <= TEXT('z'));
	};

	/** Get the canonical version of the given culture code (will also sanitize it) */
	FString GetCanonicalCultureName(const FString& Name, const FString& FallbackCulture, FInternationalization& I18N);
}
