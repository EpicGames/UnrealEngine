// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "Templates/SharedPointer.h"
#include "Internationalization/IBreakIterator.h"
#include "Internationalization/BreakIterator.h"
#include "Internationalization/CamelCaseBreakIterator.h"

#if !UE_ENABLE_ICU

class FLegacyCamelCaseBreakIterator : public FCamelCaseBreakIterator
{
protected:
	virtual void TokenizeString(FTokensArray& OutTokens) override;
};

void FLegacyCamelCaseBreakIterator::TokenizeString(FTokensArray& OutTokens)
{
	OutTokens.Empty(String.Len());

	for(int32 CurrentCharIndex = 0; CurrentCharIndex < String.Len(); ++CurrentCharIndex)
	{
		const TCHAR CurrentChar = String[CurrentCharIndex];

		ETokenType TokenType = ETokenType::Other;
		if(FChar::IsLower(CurrentChar))
		{
			TokenType = ETokenType::Lowercase;
		}
		else if(FChar::IsUpper(CurrentChar))
		{
			TokenType = ETokenType::Uppercase;
		}
		else if(FChar::IsDigit(CurrentChar))
		{
			TokenType = ETokenType::Digit;
		}

		OutTokens.Emplace(FToken(TokenType, CurrentCharIndex));
	}

	OutTokens.Emplace(FToken(ETokenType::Null, String.Len()));

	// There should always be at least one token for the end of the string
	check(OutTokens.Num());
}

TSharedRef<IBreakIterator> FBreakIterator::CreateCamelCaseBreakIterator()
{
	return MakeShareable(new FLegacyCamelCaseBreakIterator());
}

#endif
