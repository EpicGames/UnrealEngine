// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "Math/UnrealMathUtility.h"
#include "Containers/UnrealString.h"
#include "Internationalization/IBreakIterator.h"
#include "Templates/SharedPointer.h"
#include "Internationalization/BreakIterator.h"
#include "Internationalization/Text.h"

#if !UE_ENABLE_ICU

class FLegacyCharacterBoundaryIterator : public IBreakIterator
{
public:
	FLegacyCharacterBoundaryIterator();

	virtual void SetString(FString&& InString) override;
	virtual void SetStringRef(FStringView InString) override;

	virtual int32 GetCurrentPosition() const override;

	virtual int32 ResetToBeginning() override;
	virtual int32 ResetToEnd() override;

	virtual int32 MoveToPrevious() override;
	virtual int32 MoveToNext() override;
	virtual int32 MoveToCandidateBefore(const int32 InIndex) override;
	virtual int32 MoveToCandidateAfter(const int32 InIndex) override;

private:
	FString InternalString;
	FStringView String;
	int32 CurrentPosition;
};

FLegacyCharacterBoundaryIterator::FLegacyCharacterBoundaryIterator()
	: CurrentPosition(0)
{
}

void FLegacyCharacterBoundaryIterator::SetString(FString&& InString)
{
	InternalString = MoveTemp(InString);
	String = InternalString;
	ResetToBeginning();
}

void FLegacyCharacterBoundaryIterator::SetStringRef(FStringView InString)
{
	InternalString.Reset();
	String = InString;
	ResetToBeginning();
}

int32 FLegacyCharacterBoundaryIterator::GetCurrentPosition() const
{
	return CurrentPosition;
}

int32 FLegacyCharacterBoundaryIterator::ResetToBeginning()
{
	return CurrentPosition = 0;
}

int32 FLegacyCharacterBoundaryIterator::ResetToEnd()
{
	return CurrentPosition = String.Len();
}

int32 FLegacyCharacterBoundaryIterator::MoveToPrevious()
{
	return MoveToCandidateBefore(CurrentPosition);
}

int32 FLegacyCharacterBoundaryIterator::MoveToNext()
{
	return MoveToCandidateAfter(CurrentPosition);
}

int32 FLegacyCharacterBoundaryIterator::MoveToCandidateBefore(const int32 InIndex)
{
	CurrentPosition = FMath::Clamp( InIndex - 1, 0, String.Len() );
	return CurrentPosition >= InIndex ? INDEX_NONE : CurrentPosition;
}

int32 FLegacyCharacterBoundaryIterator::MoveToCandidateAfter(const int32 InIndex)
{
	CurrentPosition = FMath::Clamp( InIndex + 1, 0, String.Len() );
	return CurrentPosition <= InIndex ? INDEX_NONE : CurrentPosition;
}

TSharedRef<IBreakIterator> FBreakIterator::CreateCharacterBoundaryIterator()
{
	return MakeShareable(new FLegacyCharacterBoundaryIterator());
}

TSharedRef<IBreakIterator> FBreakIterator::GetCharacterBoundaryIterator()
{
	static thread_local TSharedPtr<IBreakIterator> CachedCharacterBoundaryIterator;
	if (!CachedCharacterBoundaryIterator.IsValid())
	{
		CachedCharacterBoundaryIterator = MakeShareable(new FLegacyCharacterBoundaryIterator());
	}

	check(CachedCharacterBoundaryIterator.IsValid());
	return CachedCharacterBoundaryIterator.ToSharedRef();
}

#endif
