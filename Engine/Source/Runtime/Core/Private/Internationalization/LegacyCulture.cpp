// Copyright Epic Games, Inc. All Rights Reserved.

#include "Internationalization/LegacyCulture.h"
#include "Internationalization/InternationalizationUtilities.h"
#include "Containers/ArrayBuilder.h"
#include "Misc/ScopeLock.h"

#if !UE_ENABLE_ICU

FLegacyCultureImplementation::FLegacyCultureImplementation(
	const FText& InDisplayName, 
	const FString& InEnglishName, 
	const int InKeyboardLayoutId, 
	const int InLCID, 
	const FString& InName, 
	const FString& InNativeName, 
	const FString& InUnrealLegacyThreeLetterISOLanguageName, 
	const FString& InThreeLetterISOLanguageName, 
	const FString& InTwoLetterISOLanguageName,
	const FDecimalNumberFormattingRules& InDecimalNumberFormattingRules,
	const FDecimalNumberFormattingRules& InPercentFormattingRules,
	const FDecimalNumberFormattingRules& InBaseCurrencyFormattingRules,
	bool InIsRightToLeft
	)
	: DisplayName(InDisplayName)
	, EnglishName(InEnglishName)
	, KeyboardLayoutId(InKeyboardLayoutId)
	, LCID(InLCID)
	, Name( InName )
	, NativeName(InNativeName)
	, UnrealLegacyThreeLetterISOLanguageName( InUnrealLegacyThreeLetterISOLanguageName )
	, ThreeLetterISOLanguageName( InThreeLetterISOLanguageName )
	, TwoLetterISOLanguageName( InTwoLetterISOLanguageName )
	, DecimalNumberFormattingRules( InDecimalNumberFormattingRules )
	, PercentFormattingRules( InPercentFormattingRules )
	, BaseCurrencyFormattingRules( InBaseCurrencyFormattingRules )
	, bIsRightToLeft(InIsRightToLeft)
{ 
}

FString FLegacyCultureImplementation::GetDisplayName() const
{
	return DisplayName.ToString();
}

FString FLegacyCultureImplementation::GetEnglishName() const
{
	return EnglishName;
}

int FLegacyCultureImplementation::GetKeyboardLayoutId() const
{
	return KeyboardLayoutId;
}

int FLegacyCultureImplementation::GetLCID() const
{
	return LCID;
}

FString FLegacyCultureImplementation::GetName() const
{
	return Name;
}

FString FLegacyCultureImplementation::GetCanonicalName(const FString& Name, FInternationalization& I18N)
{
	if (!Name.IsEmpty())
	{
		return InternationalizationUtilities::GetCanonicalCultureName(Name, FString(), I18N);
	}
	return Name;
}

FString FLegacyCultureImplementation::GetNativeName() const
{
	return NativeName;
}

FString FLegacyCultureImplementation::GetNativeLanguage() const
{
	int32 LastBracket = INDEX_NONE;
	int32 FirstBracket = INDEX_NONE;
	if ( NativeName.FindLastChar( TEXT(')'), LastBracket ) && NativeName.FindChar( TEXT('('), FirstBracket ) && LastBracket != FirstBracket )
	{
		return NativeName.Left( FirstBracket-1 );
	}
	return NativeName;
}

FString FLegacyCultureImplementation::GetNativeRegion() const
{
	int32 LastBracket = INDEX_NONE;
	int32 FirstBracket = INDEX_NONE;
	if ( NativeName.FindLastChar( TEXT(')'), LastBracket ) && NativeName.FindChar( TEXT('('), FirstBracket ) && LastBracket != FirstBracket )
	{
		return NativeName.Mid( FirstBracket+1, LastBracket-FirstBracket-1 );
	}
	return NativeName;
}

FString FLegacyCultureImplementation::GetRegion() const
{
	return FString();
}

FString FLegacyCultureImplementation::GetScript() const
{
	return FString();
}

FString FLegacyCultureImplementation::GetVariant() const
{
	return FString();
}

bool FLegacyCultureImplementation::IsRightToLeft() const
{
	return bIsRightToLeft;
}

FString FLegacyCultureImplementation::GetUnrealLegacyThreeLetterISOLanguageName() const
{
	return UnrealLegacyThreeLetterISOLanguageName;
}

FString FLegacyCultureImplementation::GetThreeLetterISOLanguageName() const
{
	return ThreeLetterISOLanguageName;
}

FString FLegacyCultureImplementation::GetTwoLetterISOLanguageName() const
{
	return TwoLetterISOLanguageName;
}

const FDecimalNumberFormattingRules& FLegacyCultureImplementation::GetDecimalNumberFormattingRules()
{
	return DecimalNumberFormattingRules;
}

const FDecimalNumberFormattingRules& FLegacyCultureImplementation::GetPercentFormattingRules()
{
	return PercentFormattingRules;
}

const FDecimalNumberFormattingRules& FLegacyCultureImplementation::GetCurrencyFormattingRules(const FString& InCurrencyCode)
{
	const bool bUseDefaultFormattingRules = InCurrencyCode.IsEmpty();

	if (bUseDefaultFormattingRules)
	{
		return BaseCurrencyFormattingRules;
	}
	else
	{
		UE::TScopeLock MapLock(UEAlternateCurrencyFormattingRulesCS);

		auto FoundUEAlternateCurrencyFormattingRules = UEAlternateCurrencyFormattingRules.FindRef(InCurrencyCode);
		if (FoundUEAlternateCurrencyFormattingRules.IsValid())
		{
			return *FoundUEAlternateCurrencyFormattingRules;
		}
	}

	FDecimalNumberFormattingRules NewUECurrencyFormattingRules = BaseCurrencyFormattingRules;
	NewUECurrencyFormattingRules.NegativePrefixString.ReplaceInline(TEXT("$"), *InCurrencyCode, ESearchCase::CaseSensitive);
	NewUECurrencyFormattingRules.NegativeSuffixString.ReplaceInline(TEXT("$"), *InCurrencyCode, ESearchCase::CaseSensitive);
	NewUECurrencyFormattingRules.PositivePrefixString.ReplaceInline(TEXT("$"), *InCurrencyCode, ESearchCase::CaseSensitive);
	NewUECurrencyFormattingRules.PositiveSuffixString.ReplaceInline(TEXT("$"), *InCurrencyCode, ESearchCase::CaseSensitive);

	{
		UE::TScopeLock MapLock(UEAlternateCurrencyFormattingRulesCS);

		// Find again in case another thread beat us to it
		auto FoundUEAlternateCurrencyFormattingRules = UEAlternateCurrencyFormattingRules.FindRef(InCurrencyCode);
		if (FoundUEAlternateCurrencyFormattingRules.IsValid())
		{
			return *FoundUEAlternateCurrencyFormattingRules;
		}

		FoundUEAlternateCurrencyFormattingRules = MakeShareable(new FDecimalNumberFormattingRules(NewUECurrencyFormattingRules));
		UEAlternateCurrencyFormattingRules.Add(InCurrencyCode, FoundUEAlternateCurrencyFormattingRules);
		return *FoundUEAlternateCurrencyFormattingRules;
	}
}

namespace
{

template <typename T>
ETextPluralForm GetDefaultPluralForm(T Val, const ETextPluralType PluralType)
{
	if (PluralType == ETextPluralType::Cardinal)
	{
		return (Val == 1) ? ETextPluralForm::One : ETextPluralForm::Other;
	}
	else
	{
		check(PluralType == ETextPluralType::Ordinal);

		if (Val % 10 == 1 && Val % 100 != 11)
		{
			return ETextPluralForm::One;
		}
		if (Val % 10 == 2 && Val % 100 != 12)
		{
			return ETextPluralForm::Two;
		}
		if (Val % 10 == 3 && Val % 100 != 13)
		{
			return ETextPluralForm::Few;
		}
	}

	return ETextPluralForm::Other;
}

}

ETextPluralForm FLegacyCultureImplementation::GetPluralForm(int32 Val, const ETextPluralType PluralType) const
{
	checkf(Val >= 0, TEXT("GetPluralFormImpl requires a positive value"));
	return GetDefaultPluralForm(Val, PluralType);
}

ETextPluralForm FLegacyCultureImplementation::GetPluralForm(double Val, const ETextPluralType PluralType) const
{
	checkf(!FMath::IsNegativeOrNegativeZero(Val), TEXT("GetPluralFormImpl requires a positive value"));
	return GetDefaultPluralForm((int64)Val, PluralType);
}

const TArray<ETextPluralForm>& FLegacyCultureImplementation::GetValidPluralForms(const ETextPluralType PluralType) const
{
	if (PluralType == ETextPluralType::Cardinal)
	{
		static const TArray<ETextPluralForm> PluralForms = TArrayBuilder<ETextPluralForm>()
			.Add(ETextPluralForm::One)
			.Add(ETextPluralForm::Other);
		return PluralForms;
	}
	else
	{
		check(PluralType == ETextPluralType::Ordinal);

		static const TArray<ETextPluralForm> PluralForms = TArrayBuilder<ETextPluralForm>()
			.Add(ETextPluralForm::One)
			.Add(ETextPluralForm::Two)
			.Add(ETextPluralForm::Few)
			.Add(ETextPluralForm::Other);
		return PluralForms;
	}
}

#endif
