// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "Containers/UnrealString.h"
#include "Containers/Map.h"
#include "Internationalization/Text.h"
#include "Internationalization/Culture.h"
#include "Internationalization/CultureImplementation.h"
#include "Internationalization/FastDecimalFormat.h"
#include "Misc/TransactionallySafeCriticalSection.h"

#if !UE_ENABLE_ICU
class FInternationalization;

class FLegacyCultureImplementation : public ICultureImplementation
{
	friend class FCulture;

public:
	FLegacyCultureImplementation(
		const FText& InDisplayName, 
		const FString& InEnglishName, 
		const int InKeyboardLayoutId, 
		const int InLCID, 
		const FString& InName, 
		const FString& InNativeName, 
		const FString& InUnrealLegacyThreeLetterISOLanguageName, 
		const FString& InThreeLetterISOLanguageName, 
		const FString& InTwoLetterISOLanguageName,
		const FDecimalNumberFormattingRules& InDecimalNumberFormattingRules,
		const FDecimalNumberFormattingRules& InPercentFormattingRules,
		const FDecimalNumberFormattingRules& InBaseCurrencyFormattingRules,
		bool InIsRightToLeft
		);
	virtual ~FLegacyCultureImplementation() = default;

	//~ ICultureImplementation interface
	virtual FString GetDisplayName() const override;
	virtual FString GetEnglishName() const override;
	virtual int GetKeyboardLayoutId() const override;
	virtual int GetLCID() const override;
	virtual FString GetName() const override;
	virtual FString GetNativeName() const override;
	virtual FString GetUnrealLegacyThreeLetterISOLanguageName() const override;
	virtual FString GetThreeLetterISOLanguageName() const override;
	virtual FString GetTwoLetterISOLanguageName() const override;
	virtual FString GetNativeLanguage() const override;
	virtual FString GetNativeRegion() const override;
	virtual FString GetRegion() const override;
	virtual FString GetScript() const override;
	virtual FString GetVariant() const override;
	virtual bool IsRightToLeft() const override;
	virtual const FDecimalNumberFormattingRules& GetDecimalNumberFormattingRules() override;
	virtual const FDecimalNumberFormattingRules& GetPercentFormattingRules() override;
	virtual const FDecimalNumberFormattingRules& GetCurrencyFormattingRules(const FString& InCurrencyCode) override;
	virtual ETextPluralForm GetPluralForm(int32 Val, const ETextPluralType PluralType) const override;
	virtual ETextPluralForm GetPluralForm(double Val, const ETextPluralType PluralType) const override;
	virtual const TArray<ETextPluralForm>& GetValidPluralForms(const ETextPluralType PluralType) const override;

	static FString GetCanonicalName(const FString& Name, FInternationalization& I18N);

private:
	// Full localized culture name
	const FText DisplayName;

	// The English name of the culture in format languagefull [country/regionfull]
	const FString EnglishName;

	// Keyboard input locale id
	const int KeyboardLayoutId;

	// id for this Culture
	const int LCID;

	// Name of the culture in languagecode2-country/regioncode2 format
	const FString Name;

	// The culture name, consisting of the language, the country/region, and the optional script
	const FString NativeName;

	// ISO 639-2 three letter code of the language - for the purpose of supporting legacy Unreal documentation.
	const FString UnrealLegacyThreeLetterISOLanguageName;

	// ISO 639-2 three letter code of the language
	const FString ThreeLetterISOLanguageName;

	// ISO 639-1 two letter code of the language
	const FString TwoLetterISOLanguageName;

	// Rules for formatting decimal numbers in this culture
	const FDecimalNumberFormattingRules DecimalNumberFormattingRules;

	// Rules for formatting percentile numbers in this culture
	const FDecimalNumberFormattingRules PercentFormattingRules;

	// Rules for formatting currency numbers in this culture
	const FDecimalNumberFormattingRules BaseCurrencyFormattingRules;

	// Is this culture right to left?
	const bool bIsRightToLeft;

	// Rules for formatting alternate currencies in this culture
	TMap<FString, TSharedPtr<const FDecimalNumberFormattingRules>> UEAlternateCurrencyFormattingRules;
	FTransactionallySafeCriticalSection UEAlternateCurrencyFormattingRulesCS;
};
#endif
