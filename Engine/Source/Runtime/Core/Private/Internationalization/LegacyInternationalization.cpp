// Copyright Epic Games, Inc. All Rights Reserved.

#include "Internationalization/LegacyInternationalization.h"
#include "Internationalization/Cultures/LeetCulture.h"
#include "Internationalization/Cultures/KeysCulture.h"

#if !UE_ENABLE_ICU

#include "Internationalization/Cultures/InvariantCulture.h"

FLegacyInternationalization::FLegacyInternationalization(FInternationalization* const InI18N)
	: I18N(InI18N)
{

}

bool FLegacyInternationalization::Initialize()
{
	I18N->InvariantCulture = FInvariantCulture::Create();
	I18N->DefaultLanguage = I18N->InvariantCulture;
	I18N->DefaultLocale = I18N->InvariantCulture;
	I18N->CurrentLanguage = I18N->InvariantCulture;
	I18N->CurrentLocale = I18N->InvariantCulture;

#if ENABLE_LOC_TESTING
	I18N->AddCustomCulture(MakeShared<FLeetCulture>(I18N->InvariantCulture.ToSharedRef()));
	I18N->AddCustomCulture(MakeShared<FKeysCulture>(I18N->InvariantCulture.ToSharedRef()));
#endif

	return true;
}

void FLegacyInternationalization::Terminate()
{
}

void FLegacyInternationalization::LoadAllCultureData()
{
}

bool FLegacyInternationalization::IsCultureRemapped(const FString& Name, FString* OutMappedCulture)
{
	return false;
}

bool FLegacyInternationalization::IsCultureAllowed(const FString& Name)
{
	return true;
}

void FLegacyInternationalization::RefreshCultureDisplayNames(const TArray<FString>& InPrioritizedDisplayCultureNames)
{
}

void FLegacyInternationalization::RefreshCachedConfigData()
{
}

void FLegacyInternationalization::HandleLanguageChanged(const FCultureRef InNewLanguage)
{
}

void FLegacyInternationalization::GetCultureNames(TArray<FString>& CultureNames) const
{
	CultureNames.Reset(1 + I18N->CustomCultures.Num());
	CultureNames.Add(FString());
	for (const FCultureRef& CustomCulture : I18N->CustomCultures)
	{
		CultureNames.Add(CustomCulture->GetName());
	}
}

TArray<FString> FLegacyInternationalization::GetPrioritizedCultureNames(const FString& Name)
{
	const FString CanonicalName = FCultureImplementation::GetCanonicalName(Name, *I18N);
	TArray<FString> PrioritizedCultureNames;
	
	if (!CanonicalName.IsEmpty())
	{
		TArray<FString> CultureFragments;
		CanonicalName.ParseIntoArray(CultureFragments, TEXT("-"), true);

		switch (CultureFragments.Num())
		{
		case 1: // Language
			PrioritizedCultureNames = FCulture::GetPrioritizedParentCultureNames(CultureFragments[0], FString(), FString());
			break;
		case 2: // Language + Region
			PrioritizedCultureNames = FCulture::GetPrioritizedParentCultureNames(CultureFragments[0], FString(), CultureFragments[1]);
			break;
		case 3: // Language + Script + Region
			PrioritizedCultureNames = FCulture::GetPrioritizedParentCultureNames(CultureFragments[0], CultureFragments[1], CultureFragments[2]);
			break;
		default:
			break;
		}
	}

	if (PrioritizedCultureNames.Num() == 0)
	{
		PrioritizedCultureNames.Add(CanonicalName);
	}

	return PrioritizedCultureNames;
}

FCulturePtr FLegacyInternationalization::GetCulture(const FString& Name)
{
	FCulturePtr Culture = I18N->GetCustomCulture(Name);
	if (!Culture && Name.IsEmpty())
	{
		Culture = I18N->InvariantCulture;
	}
	return Culture;
}

#endif
