// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Internationalization/CulturePointer.h"

class FInternationalization;

#if !UE_ENABLE_ICU

class FLegacyInternationalization
{
public:
	FLegacyInternationalization(FInternationalization* const I18N);

	bool Initialize();
	void Terminate();

	void LoadAllCultureData();

	bool IsCultureRemapped(const FString& Name, FString* OutMappedCulture);
	bool IsCultureAllowed(const FString& Name);

	void RefreshCultureDisplayNames(const TArray<FString>& InPrioritizedDisplayCultureNames);
	void RefreshCachedConfigData();

	void HandleLanguageChanged(const FCultureRef InNewLanguage);
	void GetCultureNames(TArray<FString>& CultureNames) const;
	TArray<FString> GetPrioritizedCultureNames(const FString& Name);
	FCulturePtr GetCulture(const FString& Name);

private:
	FInternationalization* const I18N;
};

#endif
