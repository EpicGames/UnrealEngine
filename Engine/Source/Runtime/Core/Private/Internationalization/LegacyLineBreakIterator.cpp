// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "Math/UnrealMathUtility.h"
#include "Containers/UnrealString.h"
#include "Internationalization/IBreakIterator.h"
#include "Templates/SharedPointer.h"
#include "Internationalization/BreakIterator.h"
#include "Internationalization/Text.h"

#if !UE_ENABLE_ICU

class FLegacyLineBreakIterator : public IBreakIterator
{
public:
	FLegacyLineBreakIterator();

	virtual void SetString(FString&& InString) override;
	virtual void SetStringRef(FStringView InString) override;

	virtual int32 GetCurrentPosition() const override;

	virtual int32 ResetToBeginning() override;
	virtual int32 ResetToEnd() override;

	virtual int32 MoveToPrevious() override;
	virtual int32 MoveToNext() override;
	virtual int32 MoveToCandidateBefore(const int32 InIndex) override;
	virtual int32 MoveToCandidateAfter(const int32 InIndex) override;

private:
	FString InternalString;
	FStringView String;
	int32 CurrentPosition;
};

FLegacyLineBreakIterator::FLegacyLineBreakIterator()
	: CurrentPosition(0)
{
}

void FLegacyLineBreakIterator::SetString(FString&& InString)
{
	InternalString = MoveTemp(InString);
	String = InternalString;
	ResetToBeginning();
}

void FLegacyLineBreakIterator::SetStringRef(FStringView InString)
{
	InternalString.Reset();
	String = InString;
	ResetToBeginning();
}

int32 FLegacyLineBreakIterator::GetCurrentPosition() const
{
	return CurrentPosition;
}

int32 FLegacyLineBreakIterator::ResetToBeginning()
{
	return CurrentPosition = 0;
}

int32 FLegacyLineBreakIterator::ResetToEnd()
{
	return CurrentPosition = String.Len();
}

int32 FLegacyLineBreakIterator::MoveToPrevious()
{
	return MoveToCandidateBefore(CurrentPosition);
}

int32 FLegacyLineBreakIterator::MoveToNext()
{
	return MoveToCandidateAfter(CurrentPosition);
}

int32 FLegacyLineBreakIterator::MoveToCandidateBefore(const int32 InIndex)
{
	// Can break between char and whitespace.
	for(CurrentPosition = FMath::Clamp( InIndex - 1, 0, String.Len() ); CurrentPosition >= 1; --CurrentPosition)
	{
		bool bPreviousCharWasWhitespace = FChar::IsWhitespace(String[CurrentPosition - 1]);
		bool bCurrentCharIsWhiteSpace = FChar::IsWhitespace(String[CurrentPosition]);
		if(bPreviousCharWasWhitespace != bCurrentCharIsWhiteSpace)
		{
			break;
		}
	}

	return CurrentPosition >= InIndex ? INDEX_NONE : CurrentPosition;
}

int32 FLegacyLineBreakIterator::MoveToCandidateAfter(const int32 InIndex)
{
	// Can break between char and whitespace.
	for(CurrentPosition = FMath::Clamp( InIndex + 1, 0, String.Len() ); CurrentPosition < String.Len(); ++CurrentPosition)
	{
		bool bPreviousCharWasWhitespace = FChar::IsWhitespace(String[CurrentPosition - 1]);
		bool bCurrentCharIsWhiteSpace = FChar::IsWhitespace(String[CurrentPosition]);
		if(bPreviousCharWasWhitespace != bCurrentCharIsWhiteSpace)
		{
			break;
		}
	}

	return (CurrentPosition <= InIndex) ? INDEX_NONE : CurrentPosition;
}

TSharedRef<IBreakIterator> FBreakIterator::CreateLineBreakIterator()
{
	return MakeShareable(new FLegacyLineBreakIterator());
}

TSharedRef<IBreakIterator> FBreakIterator::GetLineBreakIterator()
{
	static thread_local TSharedPtr<IBreakIterator> CachedLineBreakIterator;
	if (!CachedLineBreakIterator.IsValid())
	{
		CachedLineBreakIterator = MakeShareable(new FLegacyLineBreakIterator());
	}

	check(CachedLineBreakIterator.IsValid());
	return CachedLineBreakIterator.ToSharedRef();
}

#endif
