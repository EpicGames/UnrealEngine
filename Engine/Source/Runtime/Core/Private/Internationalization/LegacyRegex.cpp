// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "Containers/UnrealString.h"

#if !UE_ENABLE_ICU
#include "Internationalization/Regex.h"

class FRegexPatternImplementation
{

};

FRegexPattern::FRegexPattern(const FString& SourceString, ERegexPatternFlags Flags /*= ERegexPatternFlags::None*/)
	: Implementation(MakeShared<FRegexPatternImplementation>())
{
}

class FRegexMatcherImplementation
{

};

FRegexMatcher::FRegexMatcher(const FRegexPattern& SourcePattern, const FString& InputString)
	: Pattern(SourcePattern)
	, Implementation(MakeShared<FRegexMatcherImplementation>())
{
}

FRegexMatcher::FRegexMatcher(FRegexPattern&& SourcePattern, const FString& InputString)
	: Pattern(MoveTemp(SourcePattern))
	, Implementation(MakeShared<FRegexMatcherImplementation>())
{
}

bool FRegexMatcher::FindNext()
{
	return false;
}

FString FRegexMatcher::ReplaceFirst(FStringView Replacement)
{
	return FString();
}

FString FRegexMatcher::ReplaceAll(FStringView Replacement)
{
	return FString();
}

int32 FRegexMatcher::GetMatchBeginning()
{
	return INDEX_NONE;
}

int32 FRegexMatcher::GetMatchEnding()
{
	return INDEX_NONE;
}

int32 FRegexMatcher::GetCaptureGroupBeginning(const int32 Index)
{
	return INDEX_NONE;
}

int32 FRegexMatcher::GetCaptureGroupEnding(const int32 Index)
{
	return INDEX_NONE;
}

FString FRegexMatcher::GetCaptureGroup(const int32 Index)
{
	return FString();
}

int32 FRegexMatcher::GetBeginLimit()
{
	return INDEX_NONE;
}

int32 FRegexMatcher::GetEndLimit()
{
	return INDEX_NONE;
}

void FRegexMatcher::SetLimits(const int32 BeginIndex, const int32 EndIndex)
{
}
#endif
