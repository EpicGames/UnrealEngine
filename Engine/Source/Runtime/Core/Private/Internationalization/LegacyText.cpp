// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "Misc/DateTime.h"
#include "Internationalization/Text.h"
#include "Internationalization/TextChar.h"
#include "Internationalization/TextChronoFormatter.h"
#include "Internationalization/TextTransformer.h"
#include "Internationalization/Internationalization.h"

#if !UE_ENABLE_ICU
#include "Internationalization/TextHistory.h"

FString FTextChronoFormatter::AsDate( const FDateTime& DateTime, const EDateTimeStyle::Type DateStyle, const FString& TimeZone, const FCulture& TargetCulture )
{
	checkf(FInternationalization::Get().IsInitialized() == true, TEXT("FInternationalization is not initialized. An FText formatting method was likely used in static object initialization - this is not supported."));
	return DateTime.ToString(TEXT("%Y.%m.%d"));
}

FString FTextChronoFormatter::AsTime( const FDateTime& DateTime, const EDateTimeStyle::Type TimeStyle, const FString& TimeZone, const FCulture& TargetCulture )
{
	checkf(FInternationalization::Get().IsInitialized() == true, TEXT("FInternationalization is not initialized. An FText formatting method was likely used in static object initialization - this is not supported."));
	return DateTime.ToString(TEXT("%H.%M.%S"));
}

FString FTextChronoFormatter::AsDateTime( const FDateTime& DateTime, const EDateTimeStyle::Type DateStyle, const EDateTimeStyle::Type TimeStyle, const FString& TimeZone, const FCulture& TargetCulture )
{
	checkf(FInternationalization::Get().IsInitialized() == true, TEXT("FInternationalization is not initialized. An FText formatting method was likely used in static object initialization - this is not supported."));
	return DateTime.ToString(TEXT("%Y.%m.%d-%H.%M.%S"));
}

FString FTextChronoFormatter::AsDateTime(const FDateTime& DateTime, const FString& CustomPattern, const FString& TimeZone, const FCulture& TargetCulture)
{
	checkf(FInternationalization::Get().IsInitialized() == true, TEXT("FInternationalization is not initialized. An FText formatting method was likely used in static object initialization - this is not supported."));
	return DateTime.ToFormattedString(*CustomPattern);
}

FString FTextTransformer::ToLower(const FString& InStr)
{
	return InStr.ToLower();
}

FString FTextTransformer::ToUpper(const FString& InStr)
{
	return InStr.ToUpper();
}

bool FTextChar::IsAlpha(const UTF32CHAR Codepoint)
{
	return FChar::IsAlpha(static_cast<TCHAR>(Codepoint));
}

bool FTextChar::IsGraph(const UTF32CHAR Codepoint)
{
	return FChar::IsGraph(static_cast<TCHAR>(Codepoint));
}

bool FTextChar::IsPrint(const UTF32CHAR Codepoint)
{
	return FChar::IsPrint(static_cast<TCHAR>(Codepoint));
}

bool FTextChar::IsPunct(const UTF32CHAR Codepoint)
{
	return FChar::IsPunct(static_cast<TCHAR>(Codepoint));
}

bool FTextChar::IsAlnum(const UTF32CHAR Codepoint)
{
	return FChar::IsAlnum(static_cast<TCHAR>(Codepoint));
}

bool FTextChar::IsDigit(const UTF32CHAR Codepoint)
{
	return FChar::IsDigit(static_cast<TCHAR>(Codepoint));
}

bool FTextChar::IsHexDigit(const UTF32CHAR Codepoint)
{
	return FChar::IsHexDigit(static_cast<TCHAR>(Codepoint));
}

bool FTextChar::IsControl(const UTF32CHAR Codepoint)
{
	return FChar::IsControl(static_cast<TCHAR>(Codepoint));
}

bool FTextChar::IsWhitespace(const UTF32CHAR Codepoint)
{
	return FChar::IsWhitespace(static_cast<TCHAR>(Codepoint));
}

bool FTextChar::IsUpper(const UTF32CHAR Codepoint)
{
	return FChar::IsUpper(static_cast<TCHAR>(Codepoint));
}

bool FTextChar::IsLower(const UTF32CHAR Codepoint)
{
	return FChar::IsLower(static_cast<TCHAR>(Codepoint));
}

UTF32CHAR FTextChar::ToLower(const UTF32CHAR Codepoint)
{
	return FChar::ToLower(static_cast<TCHAR>(Codepoint));
}

UTF32CHAR FTextChar::ToUpper(const UTF32CHAR Codepoint)
{
	return FChar::ToUpper(static_cast<TCHAR>(Codepoint));
}

int32 FTextComparison::CompareTo( const FString& A, const FString& B, const ETextComparisonLevel::Type ComparisonLevel )
{
	return A.Compare(B, ESearchCase::CaseSensitive);
}

int32 FTextComparison::CompareToCaseIgnored( const FString& A, const FString& B )
{
	return A.Compare(B, ESearchCase::IgnoreCase);
}

bool FTextComparison::EqualTo( const FString& A, const FString& B, const ETextComparisonLevel::Type ComparisonLevel )
{
	return A.Equals(B, ESearchCase::CaseSensitive);
}

bool FTextComparison::EqualToCaseIgnored( const FString& A, const FString& B )
{
	return A.Equals(B, ESearchCase::IgnoreCase);
}

FText::FSortPredicate::FSortPredicate(const ETextComparisonLevel::Type ComparisonLevel)
{

}

bool FText::FSortPredicate::operator()(const FText& A, const FText& B) const
{
	return A.ToString() < B.ToString();
}

namespace TextBiDi
{

namespace Internal
{

class FLegacyTextBiDi : public ITextBiDi
{
public:
	virtual ETextDirection ComputeTextDirection(const FText& InText) override
	{
		return FLegacyTextBiDi::ComputeTextDirection(InText.ToString());
	}

	virtual ETextDirection ComputeTextDirection(const FString& InString) override
	{
		return FLegacyTextBiDi::ComputeTextDirection(*InString, 0, InString.Len());
	}

	virtual ETextDirection ComputeTextDirection(const TCHAR*, const int32 InStringStartIndex, const int32 InStringLen) override
	{
		return ETextDirection::LeftToRight;
	}

	virtual ETextDirection ComputeTextDirection(const FText& InText, const ETextDirection InBaseDirection, TArray<FTextDirectionInfo>& OutTextDirectionInfo) override
	{
		return FLegacyTextBiDi::ComputeTextDirection(InText.ToString(), InBaseDirection, OutTextDirectionInfo);
	}

	virtual ETextDirection ComputeTextDirection(const FString& InString, const ETextDirection InBaseDirection, TArray<FTextDirectionInfo>& OutTextDirectionInfo) override
	{
		return FLegacyTextBiDi::ComputeTextDirection(*InString, 0, InString.Len(), InBaseDirection, OutTextDirectionInfo);
	}

	virtual ETextDirection ComputeTextDirection(const TCHAR*, const int32 InStringStartIndex, const int32 InStringLen, const ETextDirection InBaseDirection, TArray<FTextDirectionInfo>& OutTextDirectionInfo) override
	{
		OutTextDirectionInfo.Reset();

		if (InStringLen > 0)
		{
			FTextDirectionInfo TextDirectionInfo;
			TextDirectionInfo.StartIndex = InStringStartIndex;
			TextDirectionInfo.Length = InStringLen;
			TextDirectionInfo.TextDirection = ETextDirection::LeftToRight;
			OutTextDirectionInfo.Add(MoveTemp(TextDirectionInfo));
		}

		return ETextDirection::LeftToRight;
	}

	virtual ETextDirection ComputeBaseDirection(const FText& InText) override
	{
		return FLegacyTextBiDi::ComputeBaseDirection(InText.ToString());
	}

	virtual ETextDirection ComputeBaseDirection(const FString& InString) override
	{
		return FLegacyTextBiDi::ComputeBaseDirection(*InString, 0, InString.Len());
	}

	virtual ETextDirection ComputeBaseDirection(const TCHAR*, const int32 InStringStartIndex, const int32 InStringLen) override
	{
		return ETextDirection::LeftToRight;
	}
};

} // namespace Internal

TUniquePtr<ITextBiDi> CreateTextBiDi()
{
	return MakeUnique<Internal::FLegacyTextBiDi>();
}

ETextDirection ComputeTextDirection(const FText& InText)
{
	return ComputeTextDirection(InText.ToString());
}

ETextDirection ComputeTextDirection(const FString& InString)
{
	return ComputeTextDirection(*InString, 0, InString.Len());
}

ETextDirection ComputeTextDirection(const TCHAR* InString, const int32 InStringStartIndex, const int32 InStringLen)
{
	return ETextDirection::LeftToRight;
}

ETextDirection ComputeTextDirection(const FText& InText, const ETextDirection InBaseDirection, TArray<FTextDirectionInfo>& OutTextDirectionInfo)
{
	return ComputeTextDirection(InText.ToString(), InBaseDirection, OutTextDirectionInfo);
}

ETextDirection ComputeTextDirection(const FString& InString, const ETextDirection InBaseDirection, TArray<FTextDirectionInfo>& OutTextDirectionInfo)
{
	return ComputeTextDirection(*InString, 0, InString.Len(), InBaseDirection, OutTextDirectionInfo);
}

ETextDirection ComputeTextDirection(const TCHAR* InString, const int32 InStringStartIndex, const int32 InStringLen, const ETextDirection InBaseDirection, TArray<FTextDirectionInfo>& OutTextDirectionInfo)
{
	OutTextDirectionInfo.Reset();

	if (InStringLen > 0)
	{
		FTextDirectionInfo TextDirectionInfo;
		TextDirectionInfo.StartIndex = InStringStartIndex;
		TextDirectionInfo.Length = InStringLen;
		TextDirectionInfo.TextDirection = ETextDirection::LeftToRight;
		OutTextDirectionInfo.Add(MoveTemp(TextDirectionInfo));
	}

	return ETextDirection::LeftToRight;
}

ETextDirection ComputeBaseDirection(const FText& InText)
{
	return ComputeBaseDirection(InText.ToString());
}

ETextDirection ComputeBaseDirection(const FString& InString)
{
	return ComputeBaseDirection(*InString, 0, InString.Len());
}

ETextDirection ComputeBaseDirection(const TCHAR* InString, const int32 InStringStartIndex, const int32 InStringLen)
{
	return ETextDirection::LeftToRight;
}

} // namespace TextBiDi

#endif
