// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "Math/UnrealMathUtility.h"
#include "Containers/UnrealString.h"
#include "Internationalization/IBreakIterator.h"
#include "Templates/SharedPointer.h"
#include "Internationalization/BreakIterator.h"
#include "Internationalization/Text.h"

#if !UE_ENABLE_ICU

class FLegacyWordBreakIterator : public IBreakIterator
{
public:
	FLegacyWordBreakIterator();

	virtual void SetString(FString&& InString) override;
	virtual void SetStringRef(FStringView InString) override;

	virtual int32 GetCurrentPosition() const override;

	virtual int32 ResetToBeginning() override;
	virtual int32 ResetToEnd() override;

	virtual int32 MoveToPrevious() override;
	virtual int32 MoveToNext() override;
	virtual int32 MoveToCandidateBefore(const int32 InIndex) override;
	virtual int32 MoveToCandidateAfter(const int32 InIndex) override;

private:
	FString InternalString;
	FStringView String;
	int32 CurrentPosition;
};

FLegacyWordBreakIterator::FLegacyWordBreakIterator()
	: CurrentPosition(0)
{
}

void FLegacyWordBreakIterator::SetString(FString&& InString)
{
	InternalString = MoveTemp(InString);
	String = InternalString;
	ResetToBeginning();
}

void FLegacyWordBreakIterator::SetStringRef(FStringView InString)
{
	InternalString.Reset();
	String = InString;
	ResetToBeginning();
}

int32 FLegacyWordBreakIterator::GetCurrentPosition() const
{
	return CurrentPosition;
}

int32 FLegacyWordBreakIterator::ResetToBeginning()
{
	return CurrentPosition = 0;
}

int32 FLegacyWordBreakIterator::ResetToEnd()
{
	return CurrentPosition = String.Len();
}

int32 FLegacyWordBreakIterator::MoveToPrevious()
{
	return MoveToCandidateBefore(CurrentPosition);
}

int32 FLegacyWordBreakIterator::MoveToNext()
{
	return MoveToCandidateAfter(CurrentPosition);
}

int32 FLegacyWordBreakIterator::MoveToCandidateBefore(const int32 InIndex)
{
	// Can break between char and whitespace.
	for(CurrentPosition = FMath::Clamp( InIndex - 1, 0, String.Len() ); CurrentPosition >= 1; --CurrentPosition)
	{
		bool bPreviousCharWasWhitespace = FChar::IsWhitespace(String[CurrentPosition - 1]);
		bool bCurrentCharIsWhiteSpace = FChar::IsWhitespace(String[CurrentPosition]);
		if(bPreviousCharWasWhitespace != bCurrentCharIsWhiteSpace)
		{
			break;
		}
	}

	return CurrentPosition >= InIndex ? INDEX_NONE : CurrentPosition;
}

int32 FLegacyWordBreakIterator::MoveToCandidateAfter(const int32 InIndex)
{
	// Can break between char and whitespace.
	for(CurrentPosition = FMath::Clamp( InIndex + 1, 0, String.Len() ); CurrentPosition < String.Len(); ++CurrentPosition)
	{
		bool bPreviousCharWasWhitespace = FChar::IsWhitespace(String[CurrentPosition - 1]);
		bool bCurrentCharIsWhiteSpace = FChar::IsWhitespace(String[CurrentPosition]);
		if(bPreviousCharWasWhitespace != bCurrentCharIsWhiteSpace)
		{
			break;
		}
	}

	return CurrentPosition <= InIndex ? INDEX_NONE : CurrentPosition;
}

TSharedRef<IBreakIterator> FBreakIterator::CreateWordBreakIterator()
{
	return MakeShareable(new FLegacyWordBreakIterator());
}


TSharedRef<IBreakIterator> FBreakIterator::GetWordBreakIterator()
{
	static thread_local TSharedPtr<IBreakIterator> CachedWordBreakIterator;
	if (!CachedWordBreakIterator.IsValid())
	{
		CachedWordBreakIterator = MakeShareable(new FLegacyWordBreakIterator());
	}

	check(CachedWordBreakIterator.IsValid());
	return CachedWordBreakIterator.ToSharedRef();
}

#endif
