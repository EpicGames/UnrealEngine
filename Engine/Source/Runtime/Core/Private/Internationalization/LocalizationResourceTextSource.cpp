// Copyright Epic Games, Inc. All Rights Reserved.

#include "Internationalization/LocalizationResourceTextSource.h"

#include "CoreTypes.h"
#include "HAL/FileManager.h"
#include "Internationalization/TextLocalizationResource.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreDelegates.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/Paths.h"
#include "Trace/Trace.h"
#include "Trace/Trace.inl"

bool FLocalizationResourceTextSource::GetNativeCultureName(const ELocalizedTextSourceCategory InCategory, FString& OutNativeCultureName)
{
	FString NativeCultureName = TextLocalizationResourceUtil::GetNativeCultureName(InCategory);
	if (!NativeCultureName.IsEmpty())
	{
		OutNativeCultureName = MoveTemp(NativeCultureName);
		return true;
	}
	return false;
}

void FLocalizationResourceTextSource::GetLocalizedCultureNames(const ELocalizationLoadFlags InLoadFlags, TSet<FString>& OutLocalizedCultureNames)
{
	TArray<FString> LocalizationPaths;
	if (EnumHasAnyFlags(InLoadFlags, ELocalizationLoadFlags::Editor))
	{
		LocalizationPaths += FPaths::GetEditorLocalizationPaths();
	}
	if (EnumHasAnyFlags(InLoadFlags, ELocalizationLoadFlags::Game))
	{
		LocalizationPaths += FPaths::GetGameLocalizationPaths();
	}
	if (EnumHasAnyFlags(InLoadFlags, ELocalizationLoadFlags::Engine))
	{
		LocalizationPaths += FPaths::GetEngineLocalizationPaths();
	}
	if (EnumHasAnyFlags(InLoadFlags, ELocalizationLoadFlags::Additional))
	{
		FCoreDelegates::GatherAdditionalLocResPathsCallback.Broadcast(LocalizationPaths);
	}

	TArray<FString> LocalizedCultureNames = TextLocalizationResourceUtil::GetLocalizedCultureNames(LocalizationPaths);
	OutLocalizedCultureNames.Append(MoveTemp(LocalizedCultureNames));
}

void FLocalizationResourceTextSource::LoadLocalizedResources(const ELocalizationLoadFlags InLoadFlags, TArrayView<const FString> InPrioritizedCultures, FTextLocalizationResource& InOutNativeResource, FTextLocalizationResource& InOutLocalizedResource)
{
	auto AppendChunkedLocalizationPaths = [this](TArray<FString>& OutLocalizationPaths)
	{
		if (ChunkIds.Num() > 0)
		{
			// Filter the list of chunked targets against the list of targets we're actually loading data for
			TArray<FString> ChunkedLocalizationTargets = GetChunkedLocalizationTargets();
			ChunkedLocalizationTargets.RemoveAll([&OutLocalizationPaths](const FString& LocalizationTarget)
			{
				// Note: We only allow game localization targets to be chunked, and the layout is assumed to follow our standard pattern (as used by the localization dashboard and FLocTextHelper)
				return !OutLocalizationPaths.Contains(FPaths::ProjectContentDir() / TEXT("Localization") / LocalizationTarget);
			});

			for (const FString& LocalizationTarget : ChunkedLocalizationTargets)
			{
				for (const int32 ChunkId : ChunkIds)
				{
					// Note: We only allow game localization targets to be chunked, and the layout is assumed to follow our standard pattern (as used by the localization dashboard and FLocTextHelper)
					const FString LocalizationTargetForChunk = TextLocalizationResourceUtil::GetLocalizationTargetNameForChunkId(LocalizationTarget, ChunkId);
					OutLocalizationPaths.Add(FPaths::ProjectContentDir() / TEXT("Localization") / LocalizationTargetForChunk);
				}
			}
		}
	};

	auto GetGameLocalizationPaths = [&AppendChunkedLocalizationPaths]()
	{
		TArray<FString> LocalizationPaths = FPaths::GetGameLocalizationPaths();
		AppendChunkedLocalizationPaths(LocalizationPaths);
		return LocalizationPaths;
	};

	// Collect the localization paths to load from.
	TArray<FString> GameNativePaths;
	TArray<FString> GameLocalizationPaths;
	if (ShouldLoadNativeGameData(InLoadFlags))
	{
		GameNativePaths += GetGameLocalizationPaths();
	}
	else if (ShouldLoadGame(InLoadFlags))
	{
		GameLocalizationPaths += GetGameLocalizationPaths();
	}

	TArray<FString> EditorNativePaths;
	TArray<FString> EditorLocalizationPaths;
	if (ShouldLoadEditor(InLoadFlags))
	{
		EditorLocalizationPaths += FPaths::GetEditorLocalizationPaths();
		EditorLocalizationPaths += FPaths::GetToolTipLocalizationPaths();

		bool bShouldUseLocalizedPropertyNames = false;
		if (!GConfig->GetBool(TEXT("Internationalization"), TEXT("ShouldUseLocalizedPropertyNames"), bShouldUseLocalizedPropertyNames, GEditorSettingsIni))
		{
			GConfig->GetBool(TEXT("Internationalization"), TEXT("ShouldUseLocalizedPropertyNames"), bShouldUseLocalizedPropertyNames, GEngineIni);
		}

		if (bShouldUseLocalizedPropertyNames)
		{
			EditorLocalizationPaths += FPaths::GetPropertyNameLocalizationPaths();
		}
		else
		{
			EditorNativePaths += FPaths::GetPropertyNameLocalizationPaths();
		}
	}

	TArray<FString> EngineLocalizationPaths;
	if (ShouldLoadEngine(InLoadFlags))
	{
		EngineLocalizationPaths += FPaths::GetEngineLocalizationPaths();
	}

	// Gather any additional paths that are unknown to the UE4 core (such as plugins)
	TArray<FString> AdditionalLocalizationPaths;
	if (ShouldLoadAdditional(InLoadFlags))
	{
		FCoreDelegates::GatherAdditionalLocResPathsCallback.Broadcast(AdditionalLocalizationPaths);
	}

	TArray<FString> PrioritizedLocalizationPaths;
	PrioritizedLocalizationPaths += GameLocalizationPaths;
	PrioritizedLocalizationPaths += EditorLocalizationPaths;
	PrioritizedLocalizationPaths += EngineLocalizationPaths;
	PrioritizedLocalizationPaths += AdditionalLocalizationPaths;

	TArray<FString> PrioritizedNativePaths;
	if (ShouldLoadNative(InLoadFlags))
	{
		PrioritizedNativePaths = PrioritizedLocalizationPaths;

		if (EditorNativePaths.Num() > 0)
		{
			for (const FString& LocalizationPath : EditorNativePaths)
			{
				PrioritizedNativePaths.AddUnique(LocalizationPath);
			}
		}
	}

	LoadLocalizedResourcesFromPaths(PrioritizedNativePaths, PrioritizedLocalizationPaths, GameNativePaths, InLoadFlags, InPrioritizedCultures, InOutNativeResource, InOutLocalizedResource);
}

UE_TRACE_EVENT_BEGIN(Cpu, LoadLocalizationResourcesForCulture, NoSync)
UE_TRACE_EVENT_FIELD(UE::Trace::WideString, Culture)
UE_TRACE_EVENT_FIELD(UE::Trace::WideString, Filename)
UE_TRACE_EVENT_END()

void FLocalizationResourceTextSource::LoadLocalizedResourcesFromPaths(TArrayView<const FString> InPrioritizedNativePaths, TArrayView<const FString> InPrioritizedLocalizationPaths, TArrayView<const FString> InGameNativePaths, const ELocalizationLoadFlags InLoadFlags, TArrayView<const FString> InPrioritizedCultures, FTextLocalizationResource& InOutNativeResource, FTextLocalizationResource& InOutLocalizedResource) const
{
	SCOPED_BOOT_TIMING("LoadLocalizedResourcesFromPaths");

	const int32 BaseResourcePriority = GetPriority() * -1; // Flip the priority as larger text source priorities are more important, but smaller text resource priorities are more important
	const int32 NativeResourcePriority = BaseResourcePriority + InPrioritizedCultures.Num(); // Native resources always prioritize below any localized ones

	static const FString PlatformLocalizationFolderName = FPaths::GetPlatformLocalizationFolderName();
	static const FString PlatformName = ANSI_TO_TCHAR(FPlatformProperties::IniPlatformName());
	auto LoadLocalizationResourcesForCulture = [](FTextLocalizationResource& InOutLocRes, const FString& InLocalizationPath, const FString& InCulture, const FString& InLocResFilename, const int32 InLocResPriority)
	{
		const TArray<FString>& DisabledLocalizationTargets = TextLocalizationResourceUtil::GetDisabledLocalizationTargets();
		if (DisabledLocalizationTargets.Num() > 0)
		{
			const FString LocalizationTargetName = FPaths::GetBaseFilename(InLocResFilename);
			if (DisabledLocalizationTargets.Contains(LocalizationTargetName))
			{
				return;
			}
		}

		{ 
			const FString PlatformAgnosticLocResFilename = InLocalizationPath / InCulture / InLocResFilename;
#if CPUPROFILERTRACE_ENABLED
			UE_TRACE_LOG_SCOPED_T(Cpu, LoadLocalizationResourcesForCulture, CpuChannel)
				<< LoadLocalizationResourcesForCulture.Culture(*InCulture)
				<< LoadLocalizationResourcesForCulture.Filename(*PlatformAgnosticLocResFilename);
#endif // CPUPROFILERTRACE_ENABLED
			InOutLocRes.LoadFromFile(PlatformAgnosticLocResFilename, InLocResPriority);
		}

		const FString PlatformSpecificLocResFilename = InLocalizationPath / PlatformLocalizationFolderName / PlatformName / InCulture / InLocResFilename;
		if (FPaths::FileExists(PlatformSpecificLocResFilename))
		{
#if CPUPROFILERTRACE_ENABLED
			UE_TRACE_LOG_SCOPED_T(Cpu, LoadLocalizationResourcesForCulture, CpuChannel)
				<< LoadLocalizationResourcesForCulture.Culture(*InCulture)
				<< LoadLocalizationResourcesForCulture.Filename(*PlatformSpecificLocResFilename);
#endif // CPUPROFILERTRACE_ENABLED
			InOutLocRes.LoadFromFile(PlatformSpecificLocResFilename, InLocResPriority);
		}
	};

	// Load the native texts first to ensure we always apply translations to a consistent base
	if (ShouldLoadNative(InLoadFlags) && InPrioritizedNativePaths.Num() > 0)
	{
		for (const FString& LocalizationPath : InPrioritizedNativePaths)
		{
			if (!IFileManager::Get().DirectoryExists(*LocalizationPath))
			{
				continue;
			}

			const FString LocMetaFilename = FPaths::GetBaseFilename(LocalizationPath) + TEXT(".locmeta");

			FTextLocalizationMetaDataResource LocMetaResource;
			if (LocMetaResource.LoadFromFile(LocalizationPath / LocMetaFilename))
			{
				// We skip loading the native text if we're transitioning to the native culture as there's no extra work that needs to be done
				if (!LocMetaResource.NativeCulture.IsEmpty() && !InPrioritizedCultures.Contains(LocMetaResource.NativeCulture))
				{
					LoadLocalizationResourcesForCulture(InOutNativeResource, LocalizationPath, LocMetaResource.NativeCulture, FPaths::GetCleanFilename(LocMetaResource.NativeLocRes), NativeResourcePriority);
				}
			}
		}
	}

	// The editor cheats and loads the games native localizations.
	if (ShouldLoadNativeGameData(InLoadFlags) && InGameNativePaths.Num() > 0)
	{
		const FString NativeGameCulture = TextLocalizationResourceUtil::GetNativeProjectCultureName();
		if (!NativeGameCulture.IsEmpty())
		{
			for (const FString& LocalizationPath : InGameNativePaths)
			{
				if (!IFileManager::Get().DirectoryExists(*LocalizationPath))
				{
					continue;
				}

				const FString LocResFilename = FPaths::GetBaseFilename(LocalizationPath) + TEXT(".locres");
				LoadLocalizationResourcesForCulture(InOutLocalizedResource, LocalizationPath, NativeGameCulture, LocResFilename, NativeResourcePriority);
			}
		}
	}

	// Read culture localization resources.
	if (InPrioritizedLocalizationPaths.Num() > 0)
	{
		for (int32 CultureIndex = 0; CultureIndex < InPrioritizedCultures.Num(); ++CultureIndex)
		{
			const FString& PrioritizedCultureName = InPrioritizedCultures[CultureIndex];
			for (const FString& LocalizationPath : InPrioritizedLocalizationPaths)
			{
				if (!IFileManager::Get().DirectoryExists(*LocalizationPath))
				{
					continue;
				}

				const FString LocResFilename = FPaths::GetBaseFilename(LocalizationPath) + TEXT(".locres");
				LoadLocalizationResourcesForCulture(InOutLocalizedResource, LocalizationPath, PrioritizedCultureName, LocResFilename, BaseResourcePriority + CultureIndex);
			}
		}
	}
}

TArray<FString> FLocalizationResourceTextSource::GetChunkedLocalizationTargets()
{
	TArray<FString> ChunkedLocalizationTargets;
	GConfig->GetArray(TEXT("/Script/UnrealEd.ProjectPackagingSettings"), TEXT("LocalizationTargetsToChunk"), ChunkedLocalizationTargets, GGameIni);

	const TArray<FString>& DisabledLocalizationTargets = TextLocalizationResourceUtil::GetDisabledLocalizationTargets();
	if (DisabledLocalizationTargets.Num() > 0)
	{
		ChunkedLocalizationTargets.RemoveAll([&DisabledLocalizationTargets](const FString& LocalizationTarget)
		{
			return DisabledLocalizationTargets.Contains(LocalizationTarget);
		});
	}
	
	return ChunkedLocalizationTargets;
}
