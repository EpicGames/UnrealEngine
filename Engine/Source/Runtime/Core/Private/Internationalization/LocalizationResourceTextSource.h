// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "Internationalization/ILocalizedTextSource.h"
#include "Internationalization/LocalizedTextSourceTypes.h"

class FTextLocalizationResource;

/**
 * Implementation of a localized text source that loads data from Localization Resource (LocRes) files.
 */
class FLocalizationResourceTextSource : public ILocalizedTextSource
{
public:
	//~ ILocalizedTextSource interface
	virtual bool GetNativeCultureName(const ELocalizedTextSourceCategory InCategory, FString& OutNativeCultureName) override;
	virtual void GetLocalizedCultureNames(const ELocalizationLoadFlags InLoadFlags, TSet<FString>& OutLocalizedCultureNames) override;
	virtual void LoadLocalizedResources(const ELocalizationLoadFlags InLoadFlags, TArrayView<const FString> InPrioritizedCultures, FTextLocalizationResource& InOutNativeResource, FTextLocalizationResource& InOutLocalizedResource) override;

	/**
	 * Load the localized resources from the LocRes files for the given cultures at the given paths into the given maps (ELocalizationLoadFlags controls which resources should be loaded).
	 */
	void LoadLocalizedResourcesFromPaths(TArrayView<const FString> InPrioritizedNativePaths, TArrayView<const FString> InPrioritizedLocalizationPaths, TArrayView<const FString> InGameNativePaths, const ELocalizationLoadFlags InLoadFlags, TArrayView<const FString> InPrioritizedCultures, FTextLocalizationResource& InOutNativeResource, FTextLocalizationResource& InOutLocalizedResource) const;

	/**
	 * Register that a chunk has been loaded that may contain chunked LocRes data.
	 */
	void RegisterChunkId(const int32 InChunkId)
	{
		ChunkIds.AddUnique(InChunkId);
	}

	/**
	 * Check whether the given chunk ID has been registered as containing chunked LocRes data.
	 */
	bool HasRegisteredChunkId(const int32 InChunkId)
	{
		return ChunkIds.Contains(InChunkId);
	}

	/**
	 * Get the list of localization targets that were chunked during cooking.
	 */
	static TArray<FString> GetChunkedLocalizationTargets();

private:
	/** Array of chunks that have been loaded that may contain chunked LocRes data */
	TArray<int32> ChunkIds;
};
