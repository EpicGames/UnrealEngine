// Copyright Epic Games, Inc. All Rights Reserved.

#include "Internationalization/PolyglotTextData.h"

#include "CoreTypes.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/TextLocalizationManager.h"
#include "Internationalization/TextLocalizationResource.h"
#include "Misc/AssertionMacros.h"

#define LOCTEXT_NAMESPACE "PolyglotTextData"

FPolyglotTextData::FPolyglotTextData(const ELocalizedTextSourceCategory& InCategory, const FString& InNamespace, const FString& InKey, const FString& InNativeString, const FString& InNativeCulture)
	: Category(InCategory)
	, NativeCulture(InNativeCulture)
	, Namespace(InNamespace)
	, Key(InKey)
	, NativeString(InNativeString)
{
	checkf(!Key.IsEmpty(), TEXT("Polyglot data cannot have an empty key! (Namespace: '%s', Key: '%s', Native String: '%s')"), *Namespace, *Key, *NativeString);
	checkf(!NativeString.IsEmpty(), TEXT("Polyglot data cannot have an empty native string! (Namespace: '%s', Key: '%s', Native String: '%s')"), *Namespace, *Key, *NativeString);
}

bool FPolyglotTextData::IsValid(FText* OutFailureReason) const
{
	if (Key.IsEmpty())
	{
		if (OutFailureReason)
		{
			*OutFailureReason = LOCTEXT("ValidationError_NoKey", "Polyglot data has no key set");
		}
		return false;
	}

	if (NativeString.IsEmpty())
	{
		if (OutFailureReason)
		{
			*OutFailureReason = LOCTEXT("ValidationError_NoNativeString", "Polyglot data has no native string set");
		}
		return false;
	}

	return true;
}

void FPolyglotTextData::SetCategory(const ELocalizedTextSourceCategory InCategory)
{
	ClearCache();
	Category = InCategory;
}

ELocalizedTextSourceCategory FPolyglotTextData::GetCategory() const
{
	return Category;
}

void FPolyglotTextData::SetNativeCulture(const FString& InNativeCulture)
{
	ClearCache();
	NativeCulture = InNativeCulture;
}

const FString& FPolyglotTextData::GetNativeCulture() const
{
	return NativeCulture;
}

FString FPolyglotTextData::ResolveNativeCulture() const
{
	if (!NativeCulture.IsEmpty())
	{
		return NativeCulture;
	}

	FString ResolvedNativeCulture = TextLocalizationResourceUtil::GetNativeCultureName(Category);
	if (ResolvedNativeCulture.IsEmpty())
	{
		ResolvedNativeCulture = TEXT("en");
	}

	return ResolvedNativeCulture;
}

TArray<FString> FPolyglotTextData::GetLocalizedCultures() const
{
	TArray<FString> LocalizedCultureNames;
	LocalizedStrings.GenerateKeyArray(LocalizedCultureNames);
	LocalizedCultureNames.Sort();
	return LocalizedCultureNames;
}

void FPolyglotTextData::SetIdentity(const FString& InNamespace, const FString& InKey)
{
	checkf(!InKey.IsEmpty(), TEXT("Polyglot data cannot have an empty key!"));

	ClearCache();
	Namespace = InNamespace;
	Key = InKey;
}

void FPolyglotTextData::GetIdentity(FString& OutNamespace, FString& OutKey) const
{
	OutNamespace = Namespace;
	OutKey = Key;
}

const FString& FPolyglotTextData::GetNamespace() const
{
	return Namespace;
}

const FString& FPolyglotTextData::GetKey() const
{
	return Key;
}

void FPolyglotTextData::SetNativeString(const FString& InNativeString)
{
	checkf(!InNativeString.IsEmpty(), TEXT("Polyglot data cannot have an empty native string!"));

	ClearCache();
	NativeString = InNativeString;
}

const FString& FPolyglotTextData::GetNativeString() const
{
	return NativeString;
}

void FPolyglotTextData::AddLocalizedString(const FString& InCulture, const FString& InLocalizedString)
{
	checkf(!InCulture.IsEmpty(), TEXT("Culture name cannot be empty!"));

	LocalizedStrings.Add(InCulture, InLocalizedString);
}

void FPolyglotTextData::RemoveLocalizedString(const FString& InCulture)
{
	checkf(!InCulture.IsEmpty(), TEXT("Culture name cannot be empty!"));

	LocalizedStrings.Remove(InCulture);
}

bool FPolyglotTextData::GetLocalizedString(const FString& InCulture, FString& OutLocalizedString) const
{
	checkf(!InCulture.IsEmpty(), TEXT("Culture name cannot be empty!"));

	if (const FString* FoundLocalizedString = LocalizedStrings.Find(InCulture))
	{
		OutLocalizedString = *FoundLocalizedString;
		return true;
	}

	return false;
}

void FPolyglotTextData::ClearLocalizedStrings()
{
	LocalizedStrings.Reset();
}

void FPolyglotTextData::IsMinimalPatch(const bool InIsMinimalPatch)
{
	bIsMinimalPatch = InIsMinimalPatch;
}

bool FPolyglotTextData::IsMinimalPatch() const
{
	return bIsMinimalPatch;
}

FText FPolyglotTextData::GetText() const
{
	if (CachedText.IsEmpty())
	{
		const_cast<FPolyglotTextData*>(this)->CacheText();
	}
	return CachedText;
}

void FPolyglotTextData::CacheText(FText* OutFailureReason)
{
	if (IsValid(OutFailureReason))
	{
		FTextLocalizationManager::Get().RegisterPolyglotTextData(*this);
		if (!FText::FindTextInLiveTable_Advanced(Namespace, Key, CachedText, &NativeString))
		{
			ClearCache();
		}
	}
	else
	{
		ClearCache();
	}
}

void FPolyglotTextData::ClearCache()
{
	CachedText = FText::GetEmpty();
}

#undef LOCTEXT_NAMESPACE
