// Copyright Epic Games, Inc. All Rights Reserved.

#include "Internationalization/PolyglotTextSource.h"

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Internationalization/PolyglotTextData.h"
#include "Internationalization/TextKey.h"
#include "Internationalization/TextLocalizationResource.h"
#include "Misc/AssertionMacros.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/ScopeLock.h"
#include "Templates/Tuple.h"
#include "Templates/UnrealTemplate.h"

bool FPolyglotTextSource::GetNativeCultureName(const ELocalizedTextSourceCategory InCategory, FString& OutNativeCultureName)
{
	FScopeLock ScopeLock(&PolyglotDataCS);

	if (FCultureInfo* CultureInfo = AvailableCultureInfo.Find(InCategory))
	{
		if (CultureInfo->NativeCultures.Num() > 0)
		{
			OutNativeCultureName = CultureInfo->NativeCultures.CreateConstIterator()->Key;
			return true;
		}
	}

	return false;
}

void FPolyglotTextSource::GetLocalizedCultureNames(const ELocalizationLoadFlags InLoadFlags, TSet<FString>& OutLocalizedCultureNames)
{
	auto AppendCulturesForCategory = [this, &OutLocalizedCultureNames](const ELocalizedTextSourceCategory InCategory)
	{
		auto AppendCulturesForMap = [&OutLocalizedCultureNames](const TMap<FString, int32>& CulturesMap)
		{
			TArray<FString> Cultures;
			CulturesMap.GenerateKeyArray(Cultures);
			OutLocalizedCultureNames.Append(MoveTemp(Cultures));
		};

		if (FCultureInfo* CultureInfo = AvailableCultureInfo.Find(InCategory))
		{
			AppendCulturesForMap(CultureInfo->NativeCultures);
			AppendCulturesForMap(CultureInfo->LocalizedCultures);
		}
	};

	FScopeLock ScopeLock(&PolyglotDataCS);

	if (EnumHasAnyFlags(InLoadFlags, ELocalizationLoadFlags::Editor))
	{
		AppendCulturesForCategory(ELocalizedTextSourceCategory::Editor);
	}
	if (EnumHasAnyFlags(InLoadFlags, ELocalizationLoadFlags::Game))
	{
		AppendCulturesForCategory(ELocalizedTextSourceCategory::Game);
	}
	if (EnumHasAnyFlags(InLoadFlags, ELocalizationLoadFlags::Engine))
	{
		AppendCulturesForCategory(ELocalizedTextSourceCategory::Engine);
	}
}

void FPolyglotTextSource::AddPolyglotDataToResource(const FPolyglotTextData& InPolyglotTextData, const ELocalizationLoadFlags InLoadFlags, TArrayView<const FString> InPrioritizedCultures, FTextLocalizationResource& InOutNativeResource, FTextLocalizationResource& InOutLocalizedResource) const
{
	auto ShouldLoadLocalizedText = [InLoadFlags, &InPolyglotTextData]() -> bool
	{
		switch (InPolyglotTextData.GetCategory())
		{
		case ELocalizedTextSourceCategory::Game:
			return ShouldLoadGame(InLoadFlags);
		case ELocalizedTextSourceCategory::Engine:
			return ShouldLoadEngine(InLoadFlags);
		case ELocalizedTextSourceCategory::Editor:
			return ShouldLoadEditor(InLoadFlags);
		default:
			checkf(false, TEXT("Unknown ELocalizedTextSourceCategory!"));
			break;
		}
		return false;
	};

	auto GetLocalizedStringForPolyglotData = [&InPolyglotTextData](TArrayView<const FString> InCulturesToCheck, FString& OutLocalizedString, int32* OutLocalizedPriority = nullptr) -> bool
	{
		for (int32 CultureIndex = 0; CultureIndex < InCulturesToCheck.Num(); ++CultureIndex)
		{
			const FString& CultureName = InCulturesToCheck[CultureIndex];

			if (InPolyglotTextData.GetLocalizedString(CultureName, OutLocalizedString))
			{
				if (OutLocalizedPriority)
				{
					*OutLocalizedPriority = CultureIndex;
				}
				return true;
			}
		}

		if (InPolyglotTextData.IsMinimalPatch())
		{
			return false;
		}

		if (OutLocalizedPriority)
		{
			*OutLocalizedPriority = 0;
		}
		OutLocalizedString = InPolyglotTextData.GetNativeString();
		return true;
	};

	const int32 BaseResourcePriority = GetPriority() * -1; // Flip the priority as larger text source priorities are more important, but smaller text resource priorities are more important
	const int32 NativeResourcePriority = BaseResourcePriority + InPrioritizedCultures.Num(); // Native resources always prioritize below any localized ones
	const int32 NormalNativeResourcePriority = ELocalizedTextSourcePriority::Normal + InPrioritizedCultures.Num();
	const FString NativeCulture = InPolyglotTextData.ResolveNativeCulture();

	// We skip loading the native text if we're transitioning to the native culture as there's no extra work that needs to be done
	if (ShouldLoadNative(InLoadFlags) && !InPrioritizedCultures.Contains(NativeCulture))
	{
		FString LocalizedString;
		if (GetLocalizedStringForPolyglotData(TArrayView<const FString>(&NativeCulture, 1), LocalizedString))
		{
			// When using bIsMinimalPatch, we need to drop the native string priority back down to Normal so that the LocRes translations are still allowed to overwrite it
			// Without this, any FPolyglotTextData that includes a replacement for the native translation will have that native translation take precedence for any languages not included in the FPolyglotTextData
			const int32 ResolvedNativeResourcePriority = InPolyglotTextData.IsMinimalPatch() ? NormalNativeResourcePriority : NativeResourcePriority;

			InOutNativeResource.AddEntry(
				InPolyglotTextData.GetNamespace(),
				InPolyglotTextData.GetKey(),
				InPolyglotTextData.GetNativeString(),
				LocalizedString,
				ResolvedNativeResourcePriority
				);
		}
	}

	if (ShouldLoadLocalizedText())
	{
		if (InPolyglotTextData.GetCategory() == ELocalizedTextSourceCategory::Game && ShouldLoadNativeGameData(InLoadFlags))
		{
			// The editor cheats and loads the native language's localizations for game data.
			FString LocalizedString;
			if (GetLocalizedStringForPolyglotData(TArrayView<const FString>(&NativeCulture, 1), LocalizedString))
			{
				InOutLocalizedResource.AddEntry(
					InPolyglotTextData.GetNamespace(),
					InPolyglotTextData.GetKey(),
					InPolyglotTextData.GetNativeString(),
					LocalizedString,
					NativeResourcePriority
					);
			}
		}
		else
		{
			// Find culture localization resource.
			int32 LocalizedPriority = 0;
			FString LocalizedString;
			if (GetLocalizedStringForPolyglotData(InPrioritizedCultures, LocalizedString, &LocalizedPriority))
			{
				InOutLocalizedResource.AddEntry(
					InPolyglotTextData.GetNamespace(),
					InPolyglotTextData.GetKey(),
					InPolyglotTextData.GetNativeString(),
					LocalizedString,
					BaseResourcePriority + LocalizedPriority
					);
			}
		}
	}
}

void FPolyglotTextSource::LoadLocalizedResources(const ELocalizationLoadFlags InLoadFlags, TArrayView<const FString> InPrioritizedCultures, FTextLocalizationResource& InOutNativeResource, FTextLocalizationResource& InOutLocalizedResource)
{
	FScopeLock ScopeLock(&PolyglotDataCS);

	for (const auto& PolyglotTextDataPair : PolyglotTextDataMap)
	{
		const FPolyglotTextData& PolyglotTextData = PolyglotTextDataPair.Value;
		AddPolyglotDataToResource(PolyglotTextData, InLoadFlags, InPrioritizedCultures, InOutNativeResource, InOutLocalizedResource);
	}
}

EQueryLocalizedResourceResult FPolyglotTextSource::QueryLocalizedResource(const ELocalizationLoadFlags InLoadFlags, TArrayView<const FString> InPrioritizedCultures, const FTextId InTextId, FTextLocalizationResource& InOutNativeResource, FTextLocalizationResource& InOutLocalizedResource)
{
	FScopeLock ScopeLock(&PolyglotDataCS);

	if (const FPolyglotTextData* PolyglotTextData = PolyglotTextDataMap.Find(InTextId))
	{
		AddPolyglotDataToResource(*PolyglotTextData, InLoadFlags, InPrioritizedCultures, InOutNativeResource, InOutLocalizedResource);
		return EQueryLocalizedResourceResult::Found;
	}

	return EQueryLocalizedResourceResult::NotFound;
}

void FPolyglotTextSource::RegisterPolyglotTextData(const FPolyglotTextData& InPolyglotTextData)
{
	check(InPolyglotTextData.IsValid());

	FScopeLock ScopeLock(&PolyglotDataCS);

	const FTextId Identity = FTextId(FTextKey(*InPolyglotTextData.GetNamespace()), FTextKey(*InPolyglotTextData.GetKey()));
	if (FPolyglotTextData* CurrentPolyglotData = PolyglotTextDataMap.Find(Identity))
	{
		UnregisterCultureNames(*CurrentPolyglotData);
		*CurrentPolyglotData = InPolyglotTextData;
		RegisterCultureNames(*CurrentPolyglotData);
	}
	else
	{
		PolyglotTextDataMap.Add(Identity, InPolyglotTextData);
		RegisterCultureNames(InPolyglotTextData);
	}
}

void FPolyglotTextSource::RegisterCultureNames(const FPolyglotTextData& InPolyglotTextData)
{
	auto IncrementCultureCount = [](TMap<FString, int32>& CulturesMap, const FString& CultureName)
	{
		if (!CultureName.IsEmpty())
		{
			int32& CultureCount = CulturesMap.FindOrAdd(CultureName);
			++CultureCount;
		}
	};

	FCultureInfo& CultureInfo = AvailableCultureInfo.FindOrAdd(InPolyglotTextData.GetCategory());

	const FString& NativeCulture = InPolyglotTextData.GetNativeCulture();
	IncrementCultureCount(CultureInfo.NativeCultures, NativeCulture);

	const TArray<FString> LocalizedCultures = InPolyglotTextData.GetLocalizedCultures();
	for (const FString& LocalizedCulture : LocalizedCultures)
	{
		IncrementCultureCount(CultureInfo.LocalizedCultures, LocalizedCulture);
	}
}

void FPolyglotTextSource::UnregisterCultureNames(const FPolyglotTextData& InPolyglotTextData)
{
	auto DecrementCultureCount = [](TMap<FString, int32>& CulturesMap, const FString& CultureName)
	{
		if (!CultureName.IsEmpty())
		{
			int32& CultureCount = CulturesMap.FindOrAdd(CultureName);
			--CultureCount;
			check(CultureCount >= 0);
			if (CultureCount == 0)
			{
				CulturesMap.Remove(CultureName);
			}
		}
	};

	FCultureInfo& CultureInfo = AvailableCultureInfo.FindOrAdd(InPolyglotTextData.GetCategory());

	const FString& NativeCulture = InPolyglotTextData.GetNativeCulture();
	DecrementCultureCount(CultureInfo.NativeCultures, NativeCulture);

	const TArray<FString> LocalizedCultures = InPolyglotTextData.GetLocalizedCultures();
	for (const FString& LocalizedCulture : LocalizedCultures)
	{
		DecrementCultureCount(CultureInfo.LocalizedCultures, LocalizedCulture);
	}
}
