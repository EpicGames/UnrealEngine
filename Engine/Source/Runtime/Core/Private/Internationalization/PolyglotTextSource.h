// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "HAL/CriticalSection.h"
#include "Internationalization/ILocalizedTextSource.h"
#include "Internationalization/LocKeyFuncs.h"
#include "Internationalization/PolyglotTextData.h" // IWYU pragma: keep

/**
 * Implementation of a localized text source that stores literal polyglot data registered at runtime.
 */
class FPolyglotTextSource : public ILocalizedTextSource
{
public:
	//~ ILocalizedTextSource interface
	virtual int32 GetPriority() const override { return ELocalizedTextSourcePriority::Highest; }
	virtual bool GetNativeCultureName(const ELocalizedTextSourceCategory InCategory, FString& OutNativeCultureName) override;
	virtual void GetLocalizedCultureNames(const ELocalizationLoadFlags InLoadFlags, TSet<FString>& OutLocalizedCultureNames) override;
	virtual void LoadLocalizedResources(const ELocalizationLoadFlags InLoadFlags, TArrayView<const FString> InPrioritizedCultures, FTextLocalizationResource& InOutNativeResource, FTextLocalizationResource& InOutLocalizedResource) override;
	virtual EQueryLocalizedResourceResult QueryLocalizedResource(const ELocalizationLoadFlags InLoadFlags, TArrayView<const FString> InPrioritizedCultures, const FTextId InTextId, FTextLocalizationResource& InOutNativeResource, FTextLocalizationResource& InOutLocalizedResource) override;

	/**
	 * Register a polyglot text data with this localized text source.
	 */
	void RegisterPolyglotTextData(const FPolyglotTextData& InPolyglotTextData);

private:
	/**
	 * Register the culture names associated with the given polyglot data (called as it is added to the map).
	 */
	void RegisterCultureNames(const FPolyglotTextData& InPolyglotTextData);

	/**
	 * Unregister the culture names associated with the given polyglot data (called as it is removed from the map).
	 */
	void UnregisterCultureNames(const FPolyglotTextData& InPolyglotTextData);

	/**
	 * Add the given polyglot data to the output localization resources (common logic to LoadLocalizedResources and QueryLocalizedResource).
	 */
	void AddPolyglotDataToResource(const FPolyglotTextData& InPolyglotTextData, const ELocalizationLoadFlags InLoadFlags, TArrayView<const FString> InPrioritizedCultures, FTextLocalizationResource& InOutNativeResource, FTextLocalizationResource& InOutLocalizedResource) const;

	struct FCultureInfo
	{
		TMap<FString, int32> NativeCultures;
		TMap<FString, int32> LocalizedCultures;
	};

	/**
	 * Mapping from a localization category to the currently available culture information.
	 */
	TMap<ELocalizedTextSourceCategory, FCultureInfo> AvailableCultureInfo;

	/**
	 * Mapping from a "{Namespace}::{Key}" string to a polyglot text data instance.
	 */
	TMap<FTextId, FPolyglotTextData> PolyglotTextDataMap;

	/**
	 * Critical section preventing concurrent access to the above data.
	 */
	mutable FCriticalSection PolyglotDataCS;
};
