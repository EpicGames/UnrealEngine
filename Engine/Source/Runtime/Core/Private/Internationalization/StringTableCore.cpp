// Copyright Epic Games, Inc. All Rights Reserved.

#include "Internationalization/StringTableCore.h"

#include "Containers/Array.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/Set.h"
#include "CoreGlobals.h"
#include "CoreTypes.h"
#include "Internationalization/LocKeyFuncs.h"
#include "Internationalization/TextLocalizationManager.h"
#include "Logging/LogCategory.h"
#include "Misc/CString.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/FileHelper.h"
#include "Misc/Parse.h"
#include "Misc/ScopeLock.h"
#include "Serialization/Archive.h"
#include "Serialization/Csv/CsvParser.h"
#include "Templates/Tuple.h"
#include "Trace/Detail/Channel.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

class UStringTable;

DEFINE_LOG_CATEGORY(LogStringTable);


IStringTableEngineBridge* IStringTableEngineBridge::InstancePtr = nullptr;
std::atomic<int8> IStringTableEngineBridge::DeferFindOrLoad(0);


namespace StringTableRedirects
{
	static TMap<FName, FName> TableIdRedirects;
	static TMap<FName, TMap<FTextKey, FTextKey>> TableKeyRedirects;
}


FStringTableEntry::FStringTableEntry()
{
}

#if WITH_EDITORONLY_DATA
FStringTableEntry::FStringTableEntry(FStringTableConstRef InOwnerTable, FString InSourceString, FTextId InDisplayStringId, FString InDevNotes)
#else
FStringTableEntry::FStringTableEntry(FStringTableConstRef InOwnerTable, FString InSourceString, FTextId InDisplayStringId)
#endif
	: OwnerTable(MoveTemp(InOwnerTable))
	, SourceString(MoveTemp(InSourceString))
	, DisplayStringId(MoveTemp(InDisplayStringId))
#if WITH_EDITORONLY_DATA
	, DevNotes(MoveTemp(InDevNotes))
#endif
{
}

bool FStringTableEntry::IsOwned() const
{
	return OwnerTable.IsValid();
}

void FStringTableEntry::Disown()
{
	OwnerTable.Reset();
}

bool FStringTableEntry::IsOwnedBy(const FStringTable& InStringTable) const
{
	return OwnerTable.Pin().Get() == &InStringTable;
}

const FString& FStringTableEntry::GetSourceString() const
{
	return SourceString;
}

FTextConstDisplayStringPtr FStringTableEntry::GetDisplayString() const
{
	if (FTextLocalizationManager::IsDisplayStringSupportEnabled())
	{
		return FTextLocalizationManager::Get().GetDisplayString(DisplayStringId.GetNamespace(), DisplayStringId.GetKey(), &SourceString);
	}
	return nullptr;
}

FTextId FStringTableEntry::GetDisplayStringId() const
{
	return DisplayStringId;
}

const FString& FStringTableEntry::GetPlaceholderSourceString()
{
	static const FString MissingSourceString = "<MISSING STRING TABLE ENTRY>";
	return MissingSourceString;
}


FStringTable::FStringTable()
	: OwnerAsset(nullptr)
	, bIsLoaded(true)
	, bIsInternal(false)
{
}

FStringTable::~FStringTable()
{
	// Make sure our entries are disowned correctly
	ClearSourceStrings();
}

UStringTable* FStringTable::GetOwnerAsset() const
{
	return OwnerAsset;
}

void FStringTable::SetOwnerAsset(UStringTable* InOwnerAsset)
{
	OwnerAsset = InOwnerAsset;
}

bool FStringTable::IsLoaded() const
{
	return bIsLoaded;
}

void FStringTable::IsLoaded(const bool bInIsLoaded)
{
	bIsLoaded = bInIsLoaded;
}

bool FStringTable::IsInternal() const
{
	return bIsInternal;
}

void FStringTable::IsInternal(const bool bInIsInternal)
{
	bIsInternal = bInIsInternal;
}

FString FStringTable::GetNamespace() const
{
	return TableNamespace.ToString();
}

void FStringTable::SetNamespace(const FTextKey& InNamespace)
{
	UE::TScopeLock KeyMappingLock(KeyMappingCS);

	if (TableNamespace != InNamespace)
	{
		TableNamespace = InNamespace;

		// Changing the namespace affects the display string IDs, so update those now
		for (auto& KeyToEntryPair : KeysToEntries)
		{
			FStringTableEntryPtr OldEntry = KeyToEntryPair.Value;
			OldEntry->Disown();

#if WITH_EDITORONLY_DATA
			KeyToEntryPair.Value = FStringTableEntry::NewStringTableEntry(AsShared(), OldEntry->GetSourceString(), FTextId(TableNamespace, KeyToEntryPair.Key), OldEntry->GetDevNotes());
#else
			KeyToEntryPair.Value = FStringTableEntry::NewStringTableEntry(AsShared(), OldEntry->GetSourceString(), FTextId(TableNamespace, KeyToEntryPair.Key));
#endif
		}
	}
}

bool FStringTable::GetSourceString(const FTextKey& InKey, FString& OutSourceString) const
{
	UE::TScopeLock KeyMappingLock(KeyMappingCS);

	FStringTableEntryPtr TableEntry = KeysToEntries.FindRef(InKey);
	if (TableEntry.IsValid())
	{
		OutSourceString = TableEntry->GetSourceString();
		return true;
	}
	return false;
}

#if WITH_EDITORONLY_DATA
void FStringTable::SetSourceString(const FTextKey& InKey, const FString& InSourceString, const FString& InDevNotes)
#else
void FStringTable::SetSourceString(const FTextKey& InKey, const FString& InSourceString)
#endif
{
	checkf(!InKey.IsEmpty(), TEXT("String table key cannot be empty!"));

	UE::TScopeLock KeyMappingLock(KeyMappingCS);
	
	FStringTableEntryPtr TableEntry = KeysToEntries.FindRef(InKey);
	if (TableEntry.IsValid())
	{
		TableEntry->Disown();
	}
	
#if WITH_EDITORONLY_DATA
	TableEntry = FStringTableEntry::NewStringTableEntry(AsShared(), InSourceString, FTextId(TableNamespace, InKey), InDevNotes);
#else
	TableEntry = FStringTableEntry::NewStringTableEntry(AsShared(), InSourceString, FTextId(TableNamespace, InKey));
#endif
	KeysToEntries.Emplace(InKey, TableEntry);
}

void FStringTable::RemoveSourceString(const FTextKey& InKey)
{
	UE::TScopeLock KeyMappingLock(KeyMappingCS);

	FStringTableEntryPtr TableEntry = KeysToEntries.FindRef(InKey);
	if (TableEntry.IsValid())
	{
		TableEntry->Disown();
		KeysToEntries.Remove(InKey);
		ClearMetaData(InKey);
	}
}

#if WITH_EDITORONLY_DATA
void FStringTable::EnumerateSourceStrings(const TFunctionRef<bool(const FString&, const FString&, const FString&)>& InEnumerator) const
{
	EnumerateKeysAndSourceStrings([&InEnumerator](const FTextKey& InKey, const FString& InSourceString, const FString& InDevNotes) -> bool
	{
		return InEnumerator(InKey.ToString(), InSourceString, InDevNotes);
	});
}

void FStringTable::EnumerateKeysAndSourceStrings(const TFunctionRef<bool(const FTextKey&, const FString&, const FString&)>& InEnumerator) const
{
	UE::TScopeLock KeyMappingLock(KeyMappingCS);

	for (const auto& KeyToEntryPair : KeysToEntries)
	{
		if (!InEnumerator(KeyToEntryPair.Key, KeyToEntryPair.Value->GetSourceString(), KeyToEntryPair.Value->GetDevNotes()))
		{
			break;
		}
	}
}
#endif

void FStringTable::EnumerateSourceStrings(const TFunctionRef<bool(const FString&, const FString&)>& InEnumerator) const
{
	EnumerateKeysAndSourceStrings([&InEnumerator](const FTextKey& InKey, const FString& InSourceString) -> bool
	{
		return InEnumerator(InKey.ToString(), InSourceString);
	});
}

void FStringTable::EnumerateKeysAndSourceStrings(const TFunctionRef<bool(const FTextKey&, const FString&)>& InEnumerator) const
{
	UE::TScopeLock KeyMappingLock(KeyMappingCS);

	for (const auto& KeyToEntryPair : KeysToEntries)
	{
		if (!InEnumerator(KeyToEntryPair.Key, KeyToEntryPair.Value->GetSourceString()))
		{
			break;
		}
	}
}

void FStringTable::ClearSourceStrings(const int32 InSlack)
{
	UE::TScopeLock KeyMappingLock(KeyMappingCS);

	for (const auto& KeyToEntryPair : KeysToEntries)
	{
		KeyToEntryPair.Value->Disown();
	}

	KeysToEntries.Empty(InSlack);

	ClearMetaData(InSlack);
}

FStringTableEntryConstPtr FStringTable::FindEntry(const FTextKey& InKey) const
{
	UE::TScopeLock KeyMappingLock(KeyMappingCS);
	return KeysToEntries.FindRef(InKey);
}

bool FStringTable::FindKey(const FStringTableEntryConstRef& InEntry, FString& OutKey) const
{
	FTextKey TmpKey;
	if (FindKey(InEntry, TmpKey))
	{
		TmpKey.ToString(OutKey);
		return true;
	}
	return false;
}

bool FStringTable::FindKey(const FStringTableEntryConstRef& InEntry, FTextKey& OutKey) const
{
	if (InEntry->IsOwnedBy(*this))
	{
		UE::TScopeLock KeyMappingLock(KeyMappingCS);

		for (const auto& KeyToEntryPair : KeysToEntries)
		{
			if (KeyToEntryPair.Value == InEntry)
			{
				OutKey = KeyToEntryPair.Key;
				return true;
			}
		}
	}
	return false;
}

FString FStringTable::GetMetaData(const FTextKey& InKey, const FName InMetaDataId) const
{
	UE::TScopeLock MetaDataLock(KeysToMetaDataCS);

	const FMetaDataMap* MetaDataMap = KeysToMetaData.Find(InKey);
	if (MetaDataMap)
	{
		return MetaDataMap->FindRef(InMetaDataId);
	}
	return FString();
}

void FStringTable::SetMetaData(const FTextKey& InKey, const FName InMetaDataId, const FString& InMetaDataValue)
{
	UE::TScopeLock MetaDataLock(KeysToMetaDataCS);

	FMetaDataMap& MetaDataMap = KeysToMetaData.FindOrAdd(InKey);
	MetaDataMap.Add(InMetaDataId, InMetaDataValue);
}

void FStringTable::RemoveMetaData(const FTextKey& InKey, const FName InMetaDataId)
{
	UE::TScopeLock MetaDataLock(KeysToMetaDataCS);

	FMetaDataMap* MetaDataMap = KeysToMetaData.Find(InKey);
	if (MetaDataMap)
	{
		MetaDataMap->Remove(InMetaDataId);
		if (MetaDataMap->Num() == 0)
		{
			KeysToMetaData.Remove(InKey);
		}
	}
}

void FStringTable::EnumerateMetaData(const FTextKey& InKey, const TFunctionRef<bool(FName, const FString&)>& InEnumerator) const
{
	UE::TScopeLock MetaDataLock(KeysToMetaDataCS);

	const FMetaDataMap* MetaDataMap = KeysToMetaData.Find(InKey);
	if (MetaDataMap)
	{
		for (const auto& IdToMetaDataPair : *MetaDataMap)
		{
			if (!InEnumerator(IdToMetaDataPair.Key, IdToMetaDataPair.Value))
			{
				break;
			}
		}
	}
}

void FStringTable::ClearMetaData(const FTextKey& InKey)
{
	UE::TScopeLock MetaDataLock(KeysToMetaDataCS);
	KeysToMetaData.Remove(InKey);
}

void FStringTable::ClearMetaData(const int32 InSlack)
{
	UE::TScopeLock MetaDataLock(KeysToMetaDataCS);
	KeysToMetaData.Empty(InSlack);
}

void FStringTable::Serialize(FArchive& Ar)
{
	UE::TScopeLock KeyMappingLock(KeyMappingCS);
	UE::TScopeLock MetaDataLock(KeysToMetaDataCS);

	TableNamespace.SerializeAsString(Ar);
#if WITH_EDITORONLY_DATA
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
#endif

	if (Ar.IsSaving())
	{
		// Save entries
		{
			int32 NumEntries = KeysToEntries.Num();
			Ar << NumEntries;

			for (const auto& KeyToEntryPair : KeysToEntries)
			{
				FTextKey Key = KeyToEntryPair.Key;
				Key.SerializeAsString(Ar);

				FString SourceString = KeyToEntryPair.Value->GetSourceString();
				Ar << SourceString;

#if WITH_EDITORONLY_DATA
				if (!Ar.IsCooking() && !Ar.IsFilterEditorOnly())
				{
					FString DevNotes = KeyToEntryPair.Value->GetDevNotes();
					Ar << DevNotes;
				}
#endif
			}
		}

		// Save meta-data
		{
			TMap<FString, FMetaDataMap, FDefaultSetAllocator, FLocKeyMapFuncs<FMetaDataMap>> TmpKeysToMetaData;
			TmpKeysToMetaData.Reserve(KeysToMetaData.Num());
			for (const auto& KeyToMetaDataPair : KeysToMetaData)
			{
				TmpKeysToMetaData.Add(KeyToMetaDataPair.Key.ToString(), KeyToMetaDataPair.Value);
			}

			Ar << TmpKeysToMetaData;
		}
	}
	else if (Ar.IsLoading())
	{
		// Load entries
		{
			int32 NumEntries = 0;
			Ar << NumEntries;

			ClearSourceStrings(NumEntries);
			for (int32 EntryIndex = 0; EntryIndex < NumEntries; ++EntryIndex)
			{
				FTextKey Key;
				Key.SerializeAsString(Ar);

				FString SourceString;
				Ar << SourceString;
#if WITH_EDITORONLY_DATA
				FString DevNotes;
				if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) >= FFortniteMainBranchObjectVersion::AddDevNotesToFText && !Ar.IsFilterEditorOnly())
				{
					Ar << DevNotes;
				}
				
				FStringTableEntryRef TableEntry = FStringTableEntry::NewStringTableEntry(AsShared(), SourceString, FTextId(TableNamespace, Key), DevNotes);
#else
				FStringTableEntryRef TableEntry = FStringTableEntry::NewStringTableEntry(AsShared(), SourceString, FTextId(TableNamespace, Key));
#endif
				KeysToEntries.Emplace(Key, TableEntry);
			}
		}

		// Load meta-data
		{
			TMap<FString, FMetaDataMap, FDefaultSetAllocator, FLocKeyMapFuncs<FMetaDataMap>> TmpKeysToMetaData;
			Ar << TmpKeysToMetaData;

			KeysToMetaData.Reset();
			KeysToMetaData.Reserve(TmpKeysToMetaData.Num());
			for (auto& TmpKeyToMetaDataPair : TmpKeysToMetaData)
			{
				KeysToMetaData.Add(TmpKeyToMetaDataPair.Key, MoveTemp(TmpKeyToMetaDataPair.Value));
			}
		}
	}

	if (KeysToEntries.Num() > 0)
	{
		Ar.ThisRequiresLocalizationGather();
	}
}

bool FStringTable::ExportStrings(const FString& InFilename) const
{
	return ExportStringsToCSVFile(InFilename);
}

FString FStringTable::ExportStringsToCSVString() const
{
	FString ExportedStrings;

	{
		UE::TScopeLock KeyMappingLock(KeyMappingCS);
		UE::TScopeLock MetaDataLock(KeysToMetaDataCS);

		// Collect meta-data column names
		TSet<FName> MetaDataColumnNames;
		for (const auto& KeyToMetaDataPair : KeysToMetaData)
		{
			for (const auto& IdToMetaDataPair : KeyToMetaDataPair.Value)
			{
				MetaDataColumnNames.Add(IdToMetaDataPair.Key);
			}
		}

		// Write header
		ExportedStrings += TEXT("Key,SourceString");
#if WITH_EDITORONLY_DATA
		ExportedStrings += TEXT(",DevNotes");
#endif
		for (const FName& MetaDataColumnName : MetaDataColumnNames)
		{
			ExportedStrings += TEXT(",");
			ExportedStrings += MetaDataColumnName.ToString();
		}
		ExportedStrings += TEXT("\n");

		// Write entries
		for (const auto& KeyToEntryPair : KeysToEntries)
		{
			FString ExportedKey = KeyToEntryPair.Key.ToString();
			ExportedKey.ReplaceCharWithEscapedCharInline();
			ExportedKey.ReplaceInline(TEXT("\""), TEXT("\"\""));

			FString ExportedSourceString = KeyToEntryPair.Value->GetSourceString().ReplaceCharWithEscapedChar();
			ExportedSourceString.ReplaceInline(TEXT("\""), TEXT("\"\""));

			ExportedStrings += TEXT("\"");
			ExportedStrings += ExportedKey;
			ExportedStrings += TEXT("\"");

			ExportedStrings += TEXT(",");

			ExportedStrings += TEXT("\"");
			ExportedStrings += ExportedSourceString;
			ExportedStrings += TEXT("\"");

#if WITH_EDITORONLY_DATA
			FString ExportedDevNotes = KeyToEntryPair.Value->GetDevNotes().ReplaceCharWithEscapedChar();
			ExportedDevNotes.ReplaceInline(TEXT("\""), TEXT("\"\""));
			
			ExportedStrings += TEXT(",");

			ExportedStrings += TEXT("\"");
			ExportedStrings += ExportedDevNotes;
			ExportedStrings += TEXT("\"");
#endif

			for (const FName& MetaDataColumnName : MetaDataColumnNames)
			{
				FString ExportedMetaData = GetMetaData(KeyToEntryPair.Key, MetaDataColumnName);
				ExportedMetaData.ReplaceInline(TEXT("\""), TEXT("\"\""));

				ExportedStrings += TEXT(",");

				ExportedStrings += TEXT("\"");
				ExportedStrings += ExportedMetaData;
				ExportedStrings += TEXT("\"");
			}

			ExportedStrings += TEXT("\n");
		}
	}

	return ExportedStrings;
}

bool FStringTable::ExportStringsToCSVFile(const FString& InFilename) const
{
	FString ExportedStrings = ExportStringsToCSVString();
	return FFileHelper::SaveStringToFile(ExportedStrings, *InFilename);
}

bool FStringTable::ImportStrings(const FString& InFilename)
{
	return ImportStringsFromCSVFile(InFilename);
}

bool FStringTable::ImportStringsFromCSVString(const FString& InCSVString, const TCHAR* InErrorContext)
{
	const FCsvParser ImportedStringsParser(InCSVString);
	const FCsvParser::FRows& Rows = ImportedStringsParser.GetRows();

	// Must have at least 2 rows (header and content)
	if (Rows.Num() <= 1)
	{
		UE_LOGF(LogStringTable, Warning, "Failed to import string table from '%ls'. Incorrect number of rows (must be at least 2).", InErrorContext);
		return false;
	}

	int32 KeyColumn = INDEX_NONE;
	int32 SourceStringColumn = INDEX_NONE;
#if WITH_EDITORONLY_DATA
	int32 DevNotesColumn = INDEX_NONE;
#endif
	TMap<FName, int32> MetaDataColumns;

	// Validate header
	{
		const TArray<const TCHAR*>& Cells = Rows[0];

		for (int32 CellIdx = 0; CellIdx < Cells.Num(); ++CellIdx)
		{
			const TCHAR* Cell = Cells[CellIdx];
			if (FCString::Stricmp(Cell, TEXT("Key")) == 0 && KeyColumn == INDEX_NONE)
			{
				KeyColumn = CellIdx;
			}
			else if(FCString::Stricmp(Cell, TEXT("SourceString")) == 0 && SourceStringColumn == INDEX_NONE)
			{
				SourceStringColumn = CellIdx;
			}
#if WITH_EDITORONLY_DATA
			else if(FCString::Stricmp(Cell, TEXT("DevNotes")) == 0 && DevNotesColumn == INDEX_NONE)
			{
				DevNotesColumn = CellIdx;
			}
#endif
			else
			{
				const FName MetaDataName = Cell;
				if (!MetaDataName.IsNone())
				{
					MetaDataColumns.Add(MetaDataName, CellIdx);
				}
			}
		}

		bool bValidHeader = true;
		if (KeyColumn == INDEX_NONE)
		{
			bValidHeader = false;
			UE_LOGF(LogStringTable, Warning, "Failed to import string table from '%ls'. Failed to find required column 'Key'.", InErrorContext);
		}
		if (SourceStringColumn == INDEX_NONE)
		{
			bValidHeader = false;
			UE_LOGF(LogStringTable, Warning, "Failed to import string table from '%ls'. Failed to find required column 'SourceString'.", InErrorContext);
		}
		if (!bValidHeader)
		{
			return false;
		}
	}

	// Import rows
	{
		UE::TScopeLock KeyMappingLock(KeyMappingCS);
		UE::TScopeLock MetaDataLock(KeysToMetaDataCS);

		ClearSourceStrings(Rows.Num() - 1);
		for (int32 RowIdx = 1; RowIdx < Rows.Num(); ++RowIdx)
		{
			const TArray<const TCHAR*>& Cells = Rows[RowIdx];

			// Must have at least an entry for the Key and SourceString columns
			if (Cells.IsValidIndex(KeyColumn) && Cells.IsValidIndex(SourceStringColumn))
			{
				FString Key = Cells[KeyColumn];
				Key = Key.ReplaceEscapedCharWithChar();

				FString SourceString = Cells[SourceStringColumn];
				SourceString = SourceString.ReplaceEscapedCharWithChar();

#if WITH_EDITORONLY_DATA
				FString DevNotes = Cells.IsValidIndex(DevNotesColumn) ? Cells[DevNotesColumn] : TEXT("");
				DevNotes = DevNotes.ReplaceEscapedCharWithChar();
				
				FStringTableEntryRef TableEntry = FStringTableEntry::NewStringTableEntry(AsShared(), SourceString, FTextId(TableNamespace, Key), MoveTemp(DevNotes));
#else
				FStringTableEntryRef TableEntry = FStringTableEntry::NewStringTableEntry(AsShared(), SourceString, FTextId(TableNamespace, Key));
#endif
				KeysToEntries.Emplace(Key, TableEntry);

				for (const auto& MetaDataColumnPair : MetaDataColumns)
				{
					if (Cells.IsValidIndex(MetaDataColumnPair.Value))
					{
						FString MetaData = Cells[MetaDataColumnPair.Value];
						MetaData = MetaData.ReplaceEscapedCharWithChar();

						if (!MetaData.IsEmpty())
						{
							FMetaDataMap& MetaDataMap = KeysToMetaData.FindOrAdd(Key);
							MetaDataMap.Add(MetaDataColumnPair.Key, MetaData);
						}
					}
				}
			}
		}
	}

	return true;
}

bool FStringTable::ImportStringsFromCSVFile(const FString& InFilename)
{
	FString ImportedStrings;
	if (!FFileHelper::LoadFileToString(ImportedStrings, *InFilename))
	{
		UE_LOGF(LogStringTable, Warning, "Failed to import string table from '%ls'. Could not open file.", *InFilename);
		return false;
	}
	return ImportStringsFromCSVString(ImportedStrings, *InFilename);
}


void FStringTableRedirects::InitStringTableRedirects()
{
	check(GConfig);

	const FConfigSection* CoreStringTableSection = GConfig->GetSection(TEXT("Core.StringTable"), false, GEngineIni);
	if (CoreStringTableSection)
	{
		for (FConfigSection::TConstIterator It(*CoreStringTableSection); It; ++It)
		{
			static const FName StringTableRedirectsName = "StringTableRedirects";
			if (It.Key() == StringTableRedirectsName)
			{
				const FString& ConfigValue = It.Value().GetValue();

				FName OldStringTable;
				FName NewStringTable;

				FString OldKey;
				FString NewKey;

				if (FParse::Value(*ConfigValue, TEXT("OldStringTable="), OldStringTable))
				{
					FParse::Value(*ConfigValue, TEXT("NewStringTable="), NewStringTable);
					UE_CLOGF(NewStringTable.IsNone(), LogStringTable, Warning, "Failed to parse string table redirect '%ls'. Missing or empty 'NewStringTable'.", *ConfigValue);

					if (!NewStringTable.IsNone())
					{
						StringTableRedirects::TableIdRedirects.Add(OldStringTable, NewStringTable);
					}
				}
				else if (FParse::Value(*ConfigValue, TEXT("StringTable="), OldStringTable))
				{
					FParse::Value(*ConfigValue, TEXT("OldKey="), OldKey);
					UE_CLOGF(OldKey.IsEmpty(), LogStringTable, Warning, "Failed to parse string table redirect '%ls'. Missing or empty 'OldKey'.", *ConfigValue);

					FParse::Value(*ConfigValue, TEXT("NewKey="), NewKey);
					UE_CLOGF(NewKey.IsEmpty(), LogStringTable, Warning, "Failed to parse string table redirect '%ls'. Missing or empty 'NewKey'.", *ConfigValue);

					if (!OldKey.IsEmpty() && !NewKey.IsEmpty())
					{
						StringTableRedirects::TableKeyRedirects.FindOrAdd(OldStringTable).Add(OldKey, NewKey);
					}
				}
				else
				{
					UE_LOGF(LogStringTable, Warning, "Failed to parse string table redirect '%ls'. Expected 'OldStringTable' and 'NewStringTable' for a table ID redirect, or 'StringTable', 'OldKey', 'NewKey' for a key redirect.", *ConfigValue);
				}
			}
		}
	}
}

void FStringTableRedirects::RedirectTableId(FName& InOutTableId)
{
	// Process the static redirect
	const FName* RedirectedTableId = StringTableRedirects::TableIdRedirects.Find(InOutTableId);
	if (RedirectedTableId)
	{
		InOutTableId = *RedirectedTableId;
	}

	// Process the asset redirect (only works if the asset is loaded)
	if (IStringTableEngineBridge::CanFindOrLoadStringTableAsset())
	{
		IStringTableEngineBridge::RedirectStringTableAsset(InOutTableId);
	}
}

void FStringTableRedirects::RedirectKey(const FName InTableId, FTextKey& InOutKey)
{
	const auto* RedirectedKeyMap = StringTableRedirects::TableKeyRedirects.Find(InTableId);
	if (RedirectedKeyMap)
	{
		const FTextKey RedirectedKey = RedirectedKeyMap->FindRef(InOutKey);
		if (!RedirectedKey.IsEmpty())
		{
			InOutKey = RedirectedKey;
		}
	}
}

void FStringTableRedirects::RedirectTableIdAndKey(FName& InOutTableId, FTextKey& InOutKey)
{
	RedirectTableId(InOutTableId);
	RedirectKey(InOutTableId, InOutKey);
}
