// Copyright Epic Games, Inc. All Rights Reserved.

#include "Internationalization/StringTableRegistry.h"
#include "Internationalization/StringTableCore.h"
#include "Misc/ScopeLock.h"
#include "Modules/ModuleManager.h"

#if WITH_EDITOR
#include "IDirectoryWatcher.h"
#include "DirectoryWatcherModule.h"
#endif // WITH_EDITOR

FStringTableRegistry& FStringTableRegistry::Get()
{
	static FStringTableRegistry Instance;
	return Instance;
}

FStringTableRegistry::FStringTableRegistry()
{
#if WITH_EDITOR
	// Commandlets and in-game don't listen for directory changes
	if (!IsRunningCommandlet() && GIsEditor)
	{
		FDirectoryWatcherModule& DirectoryWatcherModule = FModuleManager::LoadModuleChecked<FDirectoryWatcherModule>("DirectoryWatcher");
		IDirectoryWatcher* DirectoryWatcher = DirectoryWatcherModule.Get();

		if (DirectoryWatcher)
		{
			DirectoryWatcher->RegisterDirectoryChangedCallback_Handle(FPaths::EngineContentDir(), IDirectoryWatcher::FDirectoryChanged::CreateRaw(this, &FStringTableRegistry::OnDirectoryChanged), EngineDirectoryWatcherHandle);

			DirectoryWatcher->RegisterDirectoryChangedCallback_Handle(FPaths::ProjectContentDir(), IDirectoryWatcher::FDirectoryChanged::CreateRaw(this, &FStringTableRegistry::OnDirectoryChanged), GameDirectoryWatcherHandle);
		}
	}
#endif // WITH_EDITOR
}

FStringTableRegistry::~FStringTableRegistry()
{
#if WITH_EDITOR
	// Commandlets and in-game don't listen for directory changes
	if (!IsRunningCommandlet() && GIsEditor)
	{
		// If the directory module is still loaded, unregister any delegates
		if (FModuleManager::Get().IsModuleLoaded("DirectoryWatcher"))
		{
			FDirectoryWatcherModule& DirectoryWatcherModule = FModuleManager::GetModuleChecked<FDirectoryWatcherModule>("DirectoryWatcher");
			IDirectoryWatcher* DirectoryWatcher = DirectoryWatcherModule.Get();

			if (DirectoryWatcher)
			{
				DirectoryWatcher->UnregisterDirectoryChangedCallback_Handle(FPaths::EngineContentDir(), EngineDirectoryWatcherHandle);
				EngineDirectoryWatcherHandle.Reset();

				DirectoryWatcher->UnregisterDirectoryChangedCallback_Handle(FPaths::ProjectContentDir(), GameDirectoryWatcherHandle);
				GameDirectoryWatcherHandle.Reset();
			}
		}
	}
#endif // WITH_EDITOR
}

void FStringTableRegistry::RegisterStringTable(const FName InTableId, FStringTableRef InTable)
{
	UE::TScopeLock RegisteredStringTablesLock(RegisteredStringTablesCS);

	checkf(!InTableId.IsNone(), TEXT("String table ID cannot be 'None'!"));
#if DO_CHECK
	{
		FStringTableConstPtr ExistingStringTable = RegisteredStringTables.FindRef(InTableId);
		checkf(!ExistingStringTable.IsValid() || IStringTableEngineBridge::IsStringTableAssetBeingReplaced(ExistingStringTable->GetOwnerAsset()), TEXT("String table ID '%s' is already in use!"), *InTableId.ToString());
	}
#endif

	RegisteredStringTables.Add(InTableId, MoveTemp(InTable));
}

bool FStringTableRegistry::UnregisterStringTable(const FName InTableId)
{
	bool bDidUnregister = false;

	{
		UE::TScopeLock RegisteredStringTablesLock(RegisteredStringTablesCS);
		bDidUnregister = RegisteredStringTables.Remove(InTableId) > 0;
	}

#if WITH_EDITOR
	{
		FScopeLock CSVFilesToWatchLock(&CSVFilesToWatchCS);
		for (auto It = CSVFilesToWatch.CreateIterator(); It; ++It)
		{
			if (It->Value == InTableId)
			{
				It.RemoveCurrent();
			}
		}
	}
#endif // WITH_EDITOR

	return bDidUnregister;
}

FStringTablePtr FStringTableRegistry::FindMutableStringTable(const FName InTableId) const
{
	UE::TScopeLock RegisteredStringTablesLock(RegisteredStringTablesCS);
	return RegisteredStringTables.FindRef(InTableId);
}

FStringTableConstPtr FStringTableRegistry::FindStringTable(const FName InTableId) const
{
	UE::TScopeLock RegisteredStringTablesLock(RegisteredStringTablesCS);
	return RegisteredStringTables.FindRef(InTableId);
}

UStringTable* FStringTableRegistry::FindStringTableAsset(const FName InTableId) const
{
	UStringTable* StringTableAsset = nullptr;
	{
		FStringTableConstPtr StringTable = FindStringTable(InTableId);
		if (StringTable.IsValid())
		{
			StringTableAsset = StringTable->GetOwnerAsset();
		}
	}
	return StringTableAsset;
}

void FStringTableRegistry::EnumerateStringTables(const TFunctionRef<bool(const FName&, const FStringTableConstRef&)>& InEnumerator) const
{
	UE::TScopeLock RegisteredStringTablesLock(RegisteredStringTablesCS);

	for (const auto& RegisteredStringTablePair : RegisteredStringTables)
	{
		if (!InEnumerator(RegisteredStringTablePair.Key, RegisteredStringTablePair.Value.ToSharedRef()))
		{
			break;
		}
	}
}

void FStringTableRegistry::LogMissingStringTableEntry(const FName InTableId, const FTextKey& InKey)
{
	UE::TScopeLock LoggedMissingEntriesLock(LoggedMissingEntriesCS);

	if (const FLocKeySet* LoggedMissingKeys = LoggedMissingEntries.Find(InTableId))
	{
		if (LoggedMissingKeys->Contains(InKey))
		{
			return;
		}
	}

	FLocKeySet& LoggedMissingKeys = LoggedMissingEntries.FindOrAdd(InTableId);
	LoggedMissingKeys.Add(InKey);

	UE_LOGF(LogStringTable, Warning, "Failed to find string table entry for '%ls' '%ls'. Did you forget to add a string table redirector?", *InTableId.ToString(), *InKey.ToString());
}

#if WITH_EDITOR
void FStringTableRegistry::OnDirectoryChanged(const TArray<FFileChangeData>& InFileChanges)
{
	FScopeLock CSVFilesToWatchLock(&CSVFilesToWatchCS);

	for (const FFileChangeData& FileChange : InFileChanges)
	{
		if (FileChange.Action == FFileChangeData::FCA_Added || FileChange.Action == FFileChangeData::FCA_Modified)
		{
			const FName TableId = CSVFilesToWatch.FindRef(FPaths::ConvertRelativePathToFull(FileChange.Filename));
			if (!TableId.IsNone())
			{
				FStringTablePtr StringTable = FindMutableStringTable(TableId);
				if (StringTable.IsValid())
				{
					if (!StringTable->ImportStringsFromCSVFile(FileChange.Filename))
					{
						UE_LOGF(LogStringTable, Warning, "Failed to import strings from '%ls'", *FileChange.Filename);
					}
				}
			}
		}
	}
}
#endif // WITH_EDITOR

void FStringTableRegistry::Internal_NewLocTable(const FName InTableId, const FTextKey& InNamespace)
{
	FStringTableRef StringTable = FStringTable::NewStringTable();
	StringTable->SetNamespace(InNamespace);

	RegisterStringTable(InTableId, StringTable);
}

void FStringTableRegistry::Internal_LocTableFromFile(const FName InTableId, const FTextKey& InNamespace, const FString& InFilePath, const FString& InRootPath)
{
	FStringTableRef StringTable = FStringTable::NewStringTable();
	StringTable->SetNamespace(InNamespace);

	const FString FullPath = InRootPath / InFilePath;
	if (!StringTable->ImportStringsFromCSVFile(FullPath))
	{
		UE_LOGF(LogStringTable, Warning, "Failed to import strings from '%ls'", *FullPath);
	}

	RegisterStringTable(InTableId, StringTable);

#if WITH_EDITOR
	{
		FScopeLock CSVFilesToWatchLock(&CSVFilesToWatchCS);
		CSVFilesToWatch.Add(FPaths::ConvertRelativePathToFull(FullPath), InTableId);
	}
#endif // WITH_EDITOR
}

#if WITH_EDITORONLY_DATA
void FStringTableRegistry::Internal_SetLocTableEntry(const FName InTableId, const FTextKey& InKey, const FString& InSourceString, const FString& InDevNotes)
#else
void FStringTableRegistry::Internal_SetLocTableEntry(const FName InTableId, const FTextKey& InKey, const FString& InSourceString)
#endif
{
	FStringTablePtr StringTable = FindMutableStringTable(InTableId);
	checkf(StringTable.IsValid(), TEXT("Attempting to add a string table entry to the unknown string table '%s'"), *InTableId.ToString());
#if WITH_EDITORONLY_DATA
	StringTable->SetSourceString(InKey, InSourceString, InDevNotes);
#else
	StringTable->SetSourceString(InKey, InSourceString);
#endif
}

void FStringTableRegistry::Internal_SetLocTableEntryMetaData(const FName InTableId, const FTextKey& InKey, const FName InMetaDataId, const FString& InMetaData)
{
	FStringTablePtr StringTable = FindMutableStringTable(InTableId);
	checkf(StringTable.IsValid(), TEXT("Attempting to add string table entry meta-data to the unknown string table '%s'"), *InTableId.ToString());

	StringTable->SetMetaData(InKey, InMetaDataId, InMetaData);
}

FText FStringTableRegistry::Internal_FindLocTableEntry(const FName InTableId, const FTextKey& InKey, const EStringTableLoadingPolicy InLoadingPolicy) const
{
	return FText(InTableId, InKey, InLoadingPolicy);
}
