// Copyright Epic Games, Inc. All Rights Reserved.

#include "Internationalization/Text.h"
#include "Algo/Transform.h"
#include "Containers/AnsiString.h"
#include "Containers/Utf8String.h"
#include "Misc/Parse.h"
#include "UObject/ObjectVersion.h"
#include "UObject/DebugSerializationFlags.h"
#include "Internationalization/Culture.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/ITextGenerator.h"
#include "Internationalization/StringTableRegistry.h"

#include "Internationalization/TextHistory.h"
#include "Misc/Guid.h"
#include "Internationalization/TextCache.h"
#include "Internationalization/TextChar.h"
#include "Internationalization/TextChronoFormatter.h"
#include "Internationalization/TextFormatter.h"
#include "Internationalization/TextTransformer.h"
#include "Internationalization/TextNamespaceUtil.h"
#include "Internationalization/FastDecimalFormat.h"
#include "Internationalization/TextGeneratorRegistry.h"

#include "Serialization/StructuredArchive.h"

#include "Serialization/CompactBinaryValue.h"

#include "UObject/EditorObjectVersion.h"
#include "UObject/UE5ReleaseStreamObjectVersion.h"
#include "HAL/PlatformProcess.h"

//DEFINE_STAT(STAT_TextFormat);

DECLARE_LOG_CATEGORY_EXTERN(LogText, Log, All);
DEFINE_LOG_CATEGORY(LogText);


#define LOCTEXT_NAMESPACE "Core.Text"

bool FTextInspector::ShouldGatherForLocalization(const FText& Text)
{
	return Text.ShouldGatherForLocalization();
}

TOptional<FString> FTextInspector::GetNamespace(const FText& Text)
{
	const FTextId TextId = FTextInspector::GetTextId(Text);
	if (!TextId.IsEmpty())
	{
		return TextId.GetNamespace().ToString();
	}
	return TOptional<FString>();
}

TOptional<FString> FTextInspector::GetKey(const FText& Text)
{
	const FTextId TextId = FTextInspector::GetTextId(Text);
	if (!TextId.IsEmpty())
	{
		return TextId.GetKey().ToString();
	}
	return TOptional<FString>();
}

FTextId FTextInspector::GetTextId(const FText& Text)
{
	return Text.GetTextData().GetTextHistory().GetTextId();
}

const FString* FTextInspector::GetSourceString(const FText& Text)
{
	return &Text.GetTextData().GetSourceString();
}

const FString& FTextInspector::GetDisplayString(const FText& Text)
{
	Text.Rebuild();
	return Text.GetTextData().GetDisplayString();
}

bool FTextInspector::GetTableIdAndKey(const FText& Text, FName& OutTableId, FString& OutKey)
{
	FTextKey TmpKey;
	if (GetTableIdAndKey(Text, OutTableId, TmpKey))
	{
		TmpKey.ToString(OutKey);
		return true;
	}

	return false;
}

bool FTextInspector::GetTableIdAndKey(const FText& Text, FName& OutTableId, FTextKey& OutKey)
{
	if (Text.IsFromStringTable())
	{
		static_cast<const FTextHistory_StringTableEntry&>(Text.TextData->GetTextHistory()).GetTableIdAndKey(OutTableId, OutKey);
		return true;
	}

	return false;
}

uint32 FTextInspector::GetFlags(const FText& Text)
{
	return Text.Flags;
}

void FTextInspector::GetHistoricFormatData(const FText& Text, TArray<FHistoricTextFormatData>& OutHistoricFormatData)
{
	Text.GetHistoricFormatData(OutHistoricFormatData);
}

bool FTextInspector::GetHistoricNumericData(const FText& Text, FHistoricTextNumericData& OutHistoricNumericData)
{
	return Text.GetHistoricNumericData(OutHistoricNumericData);
}

const void* FTextInspector::GetSharedDataId(const FText& Text)
{
	return &Text.GetTextData();
}

// These default values have been duplicated to the KismetTextLibrary functions for Blueprints. Please replicate any changes there!
FNumberFormattingOptions::FNumberFormattingOptions()
	: AlwaysSign(false)
	, UseGrouping(true)
	, IndicateNearlyInteger(false)
	, RoundingMode(ERoundingMode::HalfToEven)
	, MinimumIntegralDigits(1)
	, MaximumIntegralDigits(DBL_MAX_10_EXP + DBL_DIG + 1)
	, MinimumFractionalDigits(0)
	, MaximumFractionalDigits(3)
{

}

void operator<<(FStructuredArchive::FSlot Slot, FNumberFormattingOptions& Value)
{
	FArchive& UnderlyingArchive = Slot.GetUnderlyingArchive();
	FStructuredArchive::FRecord Record = Slot.EnterRecord();
	UnderlyingArchive.UsingCustomVersion(FEditorObjectVersion::GUID);

	if (UnderlyingArchive.CustomVer(FEditorObjectVersion::GUID) >= FEditorObjectVersion::AddedAlwaysSignNumberFormattingOption)
	{
		Record << SA_VALUE(TEXT("AlwaysSign"), Value.AlwaysSign);
	}

	Record << SA_VALUE(TEXT("UseGrouping"), Value.UseGrouping);

	int8 RoundingModeInt8 = (int8)Value.RoundingMode;
	Record << SA_VALUE(TEXT("RoundingMode"), RoundingModeInt8);
	Value.RoundingMode = (ERoundingMode)RoundingModeInt8;

	Record << SA_VALUE(TEXT("MinimumIntegralDigits"), Value.MinimumIntegralDigits);
	Record << SA_VALUE(TEXT("MaximumIntegralDigits"), Value.MaximumIntegralDigits);
	Record << SA_VALUE(TEXT("MinimumFractionalDigits"), Value.MinimumFractionalDigits);
	Record << SA_VALUE(TEXT("MaximumFractionalDigits"), Value.MaximumFractionalDigits);
}

uint32 GetTypeHash( const FNumberFormattingOptions& Key )
{
	uint32 Hash = 0;
	Hash = HashCombine(Hash, GetTypeHash(Key.AlwaysSign));
	Hash = HashCombine(Hash, GetTypeHash(Key.UseGrouping));
	Hash = HashCombine(Hash, GetTypeHash(Key.RoundingMode));
	Hash = HashCombine(Hash, GetTypeHash(Key.MinimumIntegralDigits));
	Hash = HashCombine(Hash, GetTypeHash(Key.MaximumIntegralDigits));
	Hash = HashCombine(Hash, GetTypeHash(Key.MinimumFractionalDigits));
	Hash = HashCombine(Hash, GetTypeHash(Key.MaximumFractionalDigits));
	return Hash;
}

bool FNumberFormattingOptions::IsIdentical( const FNumberFormattingOptions& Other ) const
{
	return AlwaysSign == Other.AlwaysSign
		&& UseGrouping == Other.UseGrouping
		&& RoundingMode == Other.RoundingMode
		&& MinimumIntegralDigits == Other.MinimumIntegralDigits
		&& MaximumIntegralDigits == Other.MaximumIntegralDigits
		&& MinimumFractionalDigits == Other.MinimumFractionalDigits
		&& MaximumFractionalDigits == Other.MaximumFractionalDigits;
}

const FNumberFormattingOptions& FNumberFormattingOptions::DefaultWithGrouping()
{
	static const FNumberFormattingOptions Options = FNumberFormattingOptions().SetUseGrouping(true);
	return Options;
}

const FNumberFormattingOptions& FNumberFormattingOptions::DefaultNoGrouping()
{
	static const FNumberFormattingOptions Options = FNumberFormattingOptions().SetUseGrouping(false);
	return Options;
}

// These default values have been duplicated to the KismetTextLibrary functions for Blueprints. Please replicate any changes there!
FNumberParsingOptions::FNumberParsingOptions()
	: UseGrouping(true)
	, InsideLimits(false)
	, UseClamping(false)
{

}

FArchive& operator<<(FArchive& Ar, FNumberParsingOptions& Value)
{
	Ar.UsingCustomVersion(FEditorObjectVersion::GUID);

	Ar << Value.UseGrouping;
	if (Ar.CustomVer(FEditorObjectVersion::GUID) >= FEditorObjectVersion::NumberParsingOptionsNumberLimitsAndClamping)
	{
		Ar << Value.InsideLimits;
		Ar << Value.UseClamping;
	}
	return Ar;
}

uint32 GetTypeHash(const FNumberParsingOptions& Key)
{
	uint32 Hash = HashCombine(GetTypeHash(Key.UseGrouping), GetTypeHash(Key.InsideLimits));
	return HashCombine(Hash, GetTypeHash(Key.UseClamping));
}

bool FNumberParsingOptions::IsIdentical(const FNumberParsingOptions& Other) const
{
	return UseGrouping == Other.UseGrouping
		&& InsideLimits == Other.InsideLimits
		&& UseClamping == Other.UseClamping;
}

const FNumberParsingOptions& FNumberParsingOptions::DefaultWithGrouping()
{
	static const FNumberParsingOptions Options = FNumberParsingOptions().SetUseGrouping(true);
	return Options;
}

const FNumberParsingOptions& FNumberParsingOptions::DefaultNoGrouping()
{
	static const FNumberParsingOptions Options = FNumberParsingOptions().SetUseGrouping(false);
	return Options;
}

FText::FText()
	: TextData(FText::GetEmpty().TextData)
	, Flags(0)
{
}

const FText& FText::GetEmpty()
{
	static const FText StaticEmptyText = FText(MakeRefCount<FTextHistory_Base>());
	return StaticEmptyText;
}

FText::FText( FString&& InSourceString )
	: TextData(MakeRefCount<FTextHistory_Base>(FTextId(), MoveTemp(InSourceString)))
	, Flags(0)
{
}

FText::FText( FName InTableId, const FTextKey& InKey, const EStringTableLoadingPolicy InLoadingPolicy )
	: TextData(MakeRefCount<FTextHistory_StringTableEntry>(InTableId, InKey, InLoadingPolicy))
	, Flags(0)
{
}

FText::FText( FString&& InSourceString, const FTextKey& InNamespace, const FTextKey& InKey, uint32 InFlags )
	: TextData(MakeRefCount<FTextHistory_Base>(FTextId(InNamespace, InKey), MoveTemp(InSourceString)))
	, Flags(InFlags)
{
}

#if WITH_EDITORONLY_DATA
FText::FText( FString&& InSourceString, const FTextKey& InNamespace, const FTextKey& InKey, FString InDevNotes, uint32 InFlags )
	: TextData(MakeRefCount<FTextHistory_Base>(FTextId(InNamespace, InKey), MoveTemp(InSourceString), MoveTemp(InDevNotes)))
	, Flags(InFlags)
{
}
#endif

FText::FText(FText&& Other)
	: TextData(MoveTemp(Other.TextData))
	, Flags(Other.Flags)
{
	// TextData must always point to something valid
	Other.TextData = FText::GetEmpty().TextData;
	Other.Flags = 0;
}

FText& FText::operator=(FText&& Other)
{
	if (this != &Other)
	{
		TextData = MoveTemp(Other.TextData);
		Flags = Other.Flags;

		// TextData must always point to something valid
		Other.TextData = FText::GetEmpty().TextData;
		Other.Flags = 0;
	}
	return *this;
}

const ITextData& FText::GetTextData() const
{
// LiveCoding can result in FText with null TextData, so this guards against them crashing while still emitting an ensure
#if WITH_EDITOR && WITH_LIVE_CODING
	return ensure(TextData) ? *TextData : *FText::GetEmpty().TextData;
#else
	return *TextData;
#endif
}

bool FText::IsEmpty() const
{
	return ToString().IsEmpty();
}

bool FText::IsEmptyOrWhitespace() const
{
	const FString& DisplayString = ToString();
	if (DisplayString.IsEmpty())
	{
		return true;
	}

	for( const TCHAR Character : DisplayString )
	{
		if (!FTextChar::IsWhitespace(Character))
		{
			return false;
		}
	}

	return true;
}

FText FText::ToLower() const
{
	FString ResultString = FTextTransformer::ToLower(ToString());

	FText Result = FText(MakeRefCount<FTextHistory_Transform>(MoveTemp(ResultString), *this, FTextHistory_Transform::ETransformType::ToLower));
	if (!GIsEditor)
	{
		Result.Flags |= ETextFlag::Transient;
	}
	return Result;
}

FText FText::ToUpper() const
{
	FString ResultString = FTextTransformer::ToUpper(ToString());

	FText Result = FText(MakeRefCount<FTextHistory_Transform>(MoveTemp(ResultString), *this, FTextHistory_Transform::ETransformType::ToUpper));
	if (!GIsEditor)
	{
		Result.Flags |= ETextFlag::Transient;
	}
	return Result;
}

FText FText::TrimPreceding( const FText& InText )
{
	const FString& CurrentString = InText.ToString();

	int32 StartPos = 0;
	while (StartPos < CurrentString.Len())
	{
		if (!FTextChar::IsWhitespace(CurrentString[StartPos]))
		{
			break;
		}

		++StartPos;
	}

	if (StartPos == 0)
	{
		// Nothing to trim!
		return InText;
	}

	// Trim the string, preserving culture invariance if set
	FString TrimmedString = CurrentString.Right(CurrentString.Len() - StartPos);
	return InText.IsCultureInvariant() ? FText::AsCultureInvariant(MoveTemp(TrimmedString)) : FText::FromString(MoveTemp(TrimmedString));
}

FText FText::TrimTrailing( const FText& InText )
{
	const FString& CurrentString = InText.ToString();

	int32 EndPos = CurrentString.Len() - 1;
	while (EndPos >= 0)
	{
		if (!FTextChar::IsWhitespace(CurrentString[EndPos]))
		{
			break;
		}

		--EndPos;
	}

	if (EndPos == CurrentString.Len() - 1)
	{
		// Nothing to trim!
		return InText;
	}

	// Trim the string, preserving culture invariance if set
	FString TrimmedString = CurrentString.Left(EndPos + 1);
	return InText.IsCultureInvariant() ? FText::AsCultureInvariant(MoveTemp(TrimmedString)) : FText::FromString(MoveTemp(TrimmedString));
}

FText FText::TrimPrecedingAndTrailing( const FText& InText )
{
	const FString& CurrentString = InText.ToString();

	int32 StartPos = 0;
	while (StartPos < CurrentString.Len())
	{
		if (!FTextChar::IsWhitespace(CurrentString[StartPos]))
		{
			break;
		}

		++StartPos;
	}

	int32 EndPos = CurrentString.Len() - 1;
	while (EndPos > StartPos)
	{
		if (!FTextChar::IsWhitespace(CurrentString[EndPos]))
		{
			break;
		}

		--EndPos;
	}

	if (StartPos == 0 && EndPos == CurrentString.Len() - 1)
	{
		// Nothing to trim!
		return InText;
	}

	// Trim the string, preserving culture invariance if set
	FString TrimmedString = CurrentString.Mid(StartPos, EndPos - StartPos + 1);
	return InText.IsCultureInvariant() ? FText::AsCultureInvariant(MoveTemp(TrimmedString)) : FText::FromString(MoveTemp(TrimmedString));
}

bool FText::IsWhitespace(const TCHAR Char)
{
	// TCHAR should either be UTF-16 or UTF-32, so we should be fine to cast it to a UTF32CHAR for the whitespace
	// check, since whitespace is never a pair of UTF-16 characters
	return FTextChar::IsWhitespace(static_cast<UTF32CHAR>(Char));
}

void FText::GetFormatPatternParameters(const FTextFormat& Fmt, TArray<FString>& ParameterNames)
{
	return Fmt.GetFormatArgumentNames(ParameterNames);
}

FText FText::Format(FTextFormat Fmt, const FFormatNamedArguments& InArguments)
{
	return FTextFormatter::Format(MoveTemp(Fmt), CopyTemp(InArguments), false, false);
}

FText FText::Format(FTextFormat Fmt, FFormatNamedArguments&& InArguments)
{
	return FTextFormatter::Format(MoveTemp(Fmt), MoveTemp(InArguments), false, false);
}

FText FText::Format(FTextFormat Fmt, const FFormatOrderedArguments& InArguments)
{
	return FTextFormatter::Format(MoveTemp(Fmt), CopyTemp(InArguments), false, false);
}

FText FText::Format(FTextFormat Fmt, FFormatOrderedArguments&& InArguments)
{
	return FTextFormatter::Format(MoveTemp(Fmt), MoveTemp(InArguments), false, false);
}

FText FText::FormatNamedImpl(FTextFormat&& Fmt, FFormatNamedArguments&& InArguments)
{
	return FTextFormatter::Format(MoveTemp(Fmt), MoveTemp(InArguments), false, false);
}

FText FText::FormatOrderedImpl(FTextFormat&& Fmt, FFormatOrderedArguments&& InArguments)
{
	return FTextFormatter::Format(MoveTemp(Fmt), MoveTemp(InArguments), false, false);
}

template <typename T>
FText TextJoinImpl(const FText& Delimiter, const TArray<T>& Args)
{
	if (Args.Num() > 0)
	{
		FText NamedFmtPattern;
		FFormatNamedArguments NamedArgs;
		{
			FString FmtPattern;

			const int32 ArgsCount = Args.Num();
			NamedArgs.Reserve(ArgsCount + 1);
			NamedArgs.Add(TEXT("Delimiter"), Delimiter);

			for (int32 ArgIndex = 0; ArgIndex < ArgsCount; ++ArgIndex)
			{
				const T& Arg = Args[ArgIndex];
				NamedArgs.Add(FString::Printf(TEXT("%d"), ArgIndex), Arg);

				FmtPattern += TEXT('{');
				FmtPattern.AppendInt(ArgIndex);
				FmtPattern += TEXT('}');

				if (ArgIndex + 1 < ArgsCount)
				{
					FmtPattern += TEXT("{Delimiter}");
				}
			}

			NamedFmtPattern = FText::AsCultureInvariant(MoveTemp(FmtPattern));
		}

		return FTextFormatter::Format(MoveTemp(NamedFmtPattern), MoveTemp(NamedArgs), false, false);
	}

	return FText::GetEmpty();
}

FText FText::Join(const FText& Delimiter, const FFormatOrderedArguments& Args)
{
	return TextJoinImpl(Delimiter, Args);
}

FText FText::Join(const FText& Delimiter, const TArray<FText>& Args)
{
	if (Args.Num() == 1)
	{
		return Args[0];
	}
	return TextJoinImpl(Delimiter, Args);
}

FText FText::FromTextGenerator(const TSharedRef<ITextGenerator>& TextGenerator)
{
	FString ResultString = TextGenerator->BuildLocalizedDisplayString();

	FText Result = FText(MakeRefCount<FTextHistory_TextGenerator>(MoveTemp(ResultString), TextGenerator));
	if (!GIsEditor)
	{
		Result.Flags |= ETextFlag::Transient;
	}
	return Result;
}

FText::FCreateTextGeneratorDelegate FText::FindRegisteredTextGenerator( FName TypeID )
{
	return FTextGeneratorRegistry::Get().FindRegisteredTextGenerator(TypeID);
}

void FText::RegisterTextGenerator( FName TypeID, FCreateTextGeneratorDelegate FactoryFunction )
{
	FTextGeneratorRegistry::Get().RegisterTextGenerator(TypeID, MoveTemp(FactoryFunction));
}

void FText::UnregisterTextGenerator( FName TypeID )
{
	FTextGeneratorRegistry::Get().UnregisterTextGenerator(TypeID);
}

/**
* Generate an FText that represents the passed number in the passed culture
*/

// on some platforms int64_t is a typedef of long.  However, UE4 typedefs int64 as long long.  Since these are distinct types, and ICU only has a constructor for int64_t, casting to int64 causes a compile error from ambiguous function call, 
// so cast to int64_t's where appropriate here to avoid problems.

#define DEF_ASNUMBER_CAST(T1, T2) FText FText::AsNumber(T1 Val, const FNumberFormattingOptions* const Options, const FCulturePtr& TargetCulture) { return FText::AsNumberTemplate<T1, T2>(Val, Options, TargetCulture); }
#define DEF_ASNUMBER(T) DEF_ASNUMBER_CAST(T, T)
DEF_ASNUMBER(float)
DEF_ASNUMBER(double)
DEF_ASNUMBER(int8)
DEF_ASNUMBER(int16)
DEF_ASNUMBER(int32)
DEF_ASNUMBER_CAST(int64, int64_t)
DEF_ASNUMBER(uint8)
DEF_ASNUMBER(uint16)
DEF_ASNUMBER_CAST(uint32, int64_t)
DEF_ASNUMBER_CAST(uint64, int64_t)
#undef DEF_ASNUMBER
#undef DEF_ASNUMBER_CAST

template<typename T1, typename T2>
FText FText::AsNumberTemplate(T1 Val, const FNumberFormattingOptions* const Options, const FCulturePtr& TargetCulture)
{
	FInternationalization& I18N = FInternationalization::Get();
	checkf(I18N.IsInitialized() == true, TEXT("FInternationalization is not initialized. An FText formatting method was likely used in static object initialization - this is not supported."));
	const FCulture& Culture = TargetCulture.IsValid() ? *TargetCulture : *I18N.GetCurrentLocale();

	const FDecimalNumberFormattingRules& FormattingRules = Culture.GetDecimalNumberFormattingRules();
	const FNumberFormattingOptions& FormattingOptions = (Options) ? *Options : FormattingRules.CultureDefaultFormattingOptions;
	FString NativeString = FastDecimalFormat::NumberToString(Val, FormattingRules, FormattingOptions);

	FText Result = FText(MakeRefCount<FTextHistory_AsNumber>(MoveTemp(NativeString), Val, Options, TargetCulture));
	if (!GIsEditor)
	{
		Result.Flags |= ETextFlag::Transient;
	}
	return Result;
}

/**
 * Generate an FText that represents the passed number as currency in the current culture
 */
#define DEF_ASCURRENCY_CAST(T1, T2) FText FText::AsCurrency(T1 Val, const FString& CurrencyCode, const FNumberFormattingOptions* const Options, const FCulturePtr& TargetCulture) { return FText::AsCurrencyTemplate<T1, T2>(Val, CurrencyCode, Options, TargetCulture); }
#define DEF_ASCURRENCY(T) DEF_ASCURRENCY_CAST(T, T)
DEF_ASCURRENCY(float)
	DEF_ASCURRENCY(double)
	DEF_ASCURRENCY(int8)
	DEF_ASCURRENCY(int16)
	DEF_ASCURRENCY(int32)
	DEF_ASCURRENCY_CAST(int64, int64_t)
	DEF_ASCURRENCY(uint8)
	DEF_ASCURRENCY(uint16)
	DEF_ASCURRENCY_CAST(uint32, int64_t)
	DEF_ASCURRENCY_CAST(uint64, int64_t)
#undef DEF_ASCURRENCY
#undef DEF_ASCURRENCY_CAST

template<typename T1, typename T2>
FText FText::AsCurrencyTemplate(T1 Val, const FString& CurrencyCode, const FNumberFormattingOptions* const Options, const FCulturePtr& TargetCulture)
{
	FInternationalization& I18N = FInternationalization::Get();
	checkf(I18N.IsInitialized() == true, TEXT("FInternationalization is not initialized. An FText formatting method was likely used in static object initialization - this is not supported."));
	const FCulture& Culture = TargetCulture.IsValid() ? *TargetCulture : *I18N.GetCurrentLocale();

	const FDecimalNumberFormattingRules& FormattingRules = Culture.GetCurrencyFormattingRules(CurrencyCode);
	const FNumberFormattingOptions& FormattingOptions = (Options) ? *Options : FormattingRules.CultureDefaultFormattingOptions;
	FString NativeString = FastDecimalFormat::NumberToString(Val, FormattingRules, FormattingOptions);

	FText Result = FText(MakeRefCount<FTextHistory_AsCurrency>(MoveTemp(NativeString), Val, CurrencyCode, Options, TargetCulture));
	if (!GIsEditor)
	{
		Result.Flags |= ETextFlag::Transient;
	}
	return Result;
}

FText FText::AsCurrencyBase(int64 BaseVal, const FString& CurrencyCode, const FCulturePtr& TargetCulture, int32 ForceDecimalPlaces)
{
	FInternationalization& I18N = FInternationalization::Get();
	checkf(I18N.IsInitialized() == true, TEXT("FInternationalization is not initialized. An FText formatting method was likely used in static object initialization - this is not supported."));
	const FCulture& Culture = TargetCulture.IsValid() ? *TargetCulture : *I18N.GetCurrentLocale();

	const FDecimalNumberFormattingRules& FormattingRules = Culture.GetCurrencyFormattingRules(CurrencyCode);
	const FNumberFormattingOptions& FormattingOptions = FormattingRules.CultureDefaultFormattingOptions;
	int32 DecimalPlaces = ForceDecimalPlaces >= 0 ? ForceDecimalPlaces : FormattingOptions.MaximumFractionalDigits;
	double Val = static_cast<double>(BaseVal) / static_cast<double>(FastDecimalFormat::Pow10(DecimalPlaces));
	FString NativeString = FastDecimalFormat::NumberToString(Val, FormattingRules, FormattingOptions);

	FText Result = FText(MakeRefCount<FTextHistory_AsCurrency>(MoveTemp(NativeString), Val, CurrencyCode, nullptr, TargetCulture));
	if (!GIsEditor)
	{
		Result.Flags |= ETextFlag::Transient;
	}
	return Result;
}

/**
* Generate an FText that represents the passed number as a percentage in the current culture
*/

#define DEF_ASPERCENT_CAST(T1, T2) FText FText::AsPercent(T1 Val, const FNumberFormattingOptions* const Options, const FCulturePtr& TargetCulture) { return FText::AsPercentTemplate<T1, T2>(Val, Options, TargetCulture); }
#define DEF_ASPERCENT(T) DEF_ASPERCENT_CAST(T, T)
DEF_ASPERCENT(double)
DEF_ASPERCENT(float)
#undef DEF_ASPERCENT
#undef DEF_ASPERCENT_CAST

template<typename T1, typename T2>
FText FText::AsPercentTemplate(T1 Val, const FNumberFormattingOptions* const Options, const FCulturePtr& TargetCulture)
{
	FInternationalization& I18N = FInternationalization::Get();
	checkf(I18N.IsInitialized() == true, TEXT("FInternationalization is not initialized. An FText formatting method was likely used in static object initialization - this is not supported."));
	const FCulture& Culture = TargetCulture.IsValid() ? *TargetCulture : *I18N.GetCurrentLocale();

	const FDecimalNumberFormattingRules& FormattingRules = Culture.GetPercentFormattingRules();
	const FNumberFormattingOptions& FormattingOptions = (Options) ? *Options : FormattingRules.CultureDefaultFormattingOptions;
	FString NativeString = FastDecimalFormat::NumberToString(Val * static_cast<T1>(100), FormattingRules, FormattingOptions);

	FText Result = FText(MakeRefCount<FTextHistory_AsPercent>(MoveTemp(NativeString), Val, Options, TargetCulture));
	if (!GIsEditor)
	{
		Result.Flags |= ETextFlag::Transient;
	}
	return Result;
}

FText FText::AsDate(const FDateTime& DateTime, const EDateTimeStyle::Type DateStyle, const FString& TimeZone, const FCulturePtr& TargetCulture)
{
	FInternationalization& I18N = FInternationalization::Get();
	checkf(I18N.IsInitialized() == true, TEXT("FInternationalization is not initialized. An FText formatting method was likely used in static object initialization - this is not supported."));
	const FCulture& Culture = TargetCulture.IsValid() ? *TargetCulture : *I18N.GetCurrentLocale();

	FString ChronoSting = FTextChronoFormatter::AsDate(DateTime, DateStyle, TimeZone, Culture);
	FText Result = FText(MakeRefCount<FTextHistory_AsDate>(MoveTemp(ChronoSting), DateTime, DateStyle, TimeZone, TargetCulture));
	if (!GIsEditor)
	{
		Result.Flags |= ETextFlag::Transient;
	}
	return Result;
}

FText FText::AsDateTime(const FDateTime& DateTime, const EDateTimeStyle::Type DateStyle, const EDateTimeStyle::Type TimeStyle, const FString& TimeZone, const FCulturePtr& TargetCulture)
{
	FInternationalization& I18N = FInternationalization::Get();
	checkf(I18N.IsInitialized() == true, TEXT("FInternationalization is not initialized. An FText formatting method was likely used in static object initialization - this is not supported."));
	const FCulture& Culture = TargetCulture.IsValid() ? *TargetCulture : *I18N.GetCurrentLocale();

	FString ChronoString = FTextChronoFormatter::AsDateTime(DateTime, DateStyle, TimeStyle, TimeZone, Culture);
	FText Result = FText(MakeRefCount<FTextHistory_AsDateTime>(MoveTemp(ChronoString), DateTime, DateStyle, TimeStyle, TimeZone, TargetCulture));
	if (!GIsEditor)
	{
		Result.Flags |= ETextFlag::Transient;
	}
	return Result;
}

FText FText::AsDateTime(const FDateTime& DateTime, const FString& CustomPattern, const FString& TimeZone, const FCulturePtr& TargetCulture)
{
	FInternationalization& I18N = FInternationalization::Get();
	checkf(I18N.IsInitialized() == true, TEXT("FInternationalization is not initialized. An FText formatting method was likely used in static object initialization - this is not supported."));
	const FCulture& Culture = TargetCulture.IsValid() ? *TargetCulture : *I18N.GetCurrentLocale();

	FString ChronoString = FTextChronoFormatter::AsDateTime(DateTime, CustomPattern, TimeZone, Culture);
	FText Result = FText(MakeRefCount<FTextHistory_AsDateTime>(MoveTemp(ChronoString), DateTime, CustomPattern, TimeZone, TargetCulture));
	if (!GIsEditor)
	{
		Result.Flags |= ETextFlag::Transient;
	}
	return Result;
}

FText FText::AsTime(const FDateTime& DateTime, const EDateTimeStyle::Type TimeStyle, const FString& TimeZone, const FCulturePtr& TargetCulture)
{
	FInternationalization& I18N = FInternationalization::Get();
	checkf(I18N.IsInitialized() == true, TEXT("FInternationalization is not initialized. An FText formatting method was likely used in static object initialization - this is not supported."));
	const FCulture& Culture = TargetCulture.IsValid() ? *TargetCulture : *I18N.GetCurrentLocale();

	FString ChronoString = FTextChronoFormatter::AsTime(DateTime, TimeStyle, TimeZone, Culture);
	FText Result = FText(MakeRefCount<FTextHistory_AsTime>(MoveTemp(ChronoString), DateTime, TimeStyle, TimeZone, TargetCulture));
	if (!GIsEditor)
	{
		Result.Flags |= ETextFlag::Transient;
	}
	return Result;
}

FText FText::AsTimespan(const FTimespan& Timespan, const FCulturePtr& TargetCulture)
{
	FInternationalization& I18N = FInternationalization::Get();
	checkf(I18N.IsInitialized() == true, TEXT("FInternationalization is not initialized. An FText formatting method was likely used in static object initialization - this is not supported."));
	FCultureRef Culture = TargetCulture.IsValid() ? TargetCulture.ToSharedRef() : I18N.GetCurrentLocale();

	double TotalHours = Timespan.GetTotalHours();
	int32 Hours = static_cast<int32>(TotalHours);
	int32 Minutes = Timespan.GetMinutes();
	int32 Seconds = Timespan.GetSeconds();

	FNumberFormattingOptions NumberFormattingOptions;
	NumberFormattingOptions.MinimumIntegralDigits = 2;
	NumberFormattingOptions.MaximumIntegralDigits = 2;

	if (Hours > 0)
	{
		FText TimespanFormatPattern = NSLOCTEXT("Timespan", "Format_HoursMinutesSeconds", "{Hours}:{Minutes}:{Seconds}");
		FFormatNamedArguments TimeArguments;
		TimeArguments.Add(TEXT("Hours"), Hours);
		TimeArguments.Add(TEXT("Minutes"), FText::AsNumber(Minutes, &NumberFormattingOptions, Culture));
		TimeArguments.Add(TEXT("Seconds"), FText::AsNumber(Seconds, &NumberFormattingOptions, Culture));
		return FText::Format(TimespanFormatPattern, TimeArguments);
	}
	else
	{
		FText TimespanFormatPattern = NSLOCTEXT("Timespan", "Format_MinutesSeconds", "{Minutes}:{Seconds}");
		FFormatNamedArguments TimeArguments;
		TimeArguments.Add(TEXT("Minutes"), Minutes);
		TimeArguments.Add(TEXT("Seconds"), FText::AsNumber(Seconds, &NumberFormattingOptions, Culture));
		return FText::Format(TimespanFormatPattern, TimeArguments);
	}
}

FText FText::AsMemory(uint64 NumBytes, const FNumberFormattingOptions* const Options, const FCulturePtr& TargetCulture, EMemoryUnitStandard UnitStandard)
{
	checkf(FInternationalization::Get().IsInitialized() == true, TEXT("FInternationalization is not initialized. An FText formatting method was likely used in static object initialization - this is not supported."));
	FFormatNamedArguments Args;

	static const TCHAR* Prefixes = TEXT("KMGTPEZYkMGTPEZY");
	int32 Prefix = 0;
	FString Suffix = "iB";
	uint64 Unit = 1024;
	if (UnitStandard == EMemoryUnitStandard::SI)
	{
		Prefix = 8;
		Suffix = TEXT("B");
		Unit = 1000;
	}

	// We consistently use decimal magnitude for testing, so that both IEC and SI remain friendly as a stringified decimal number.
	if (NumBytes < 1000)
	{
		Args.Add( TEXT("Number"), FText::AsNumber( NumBytes, Options, TargetCulture) );
		Args.Add( TEXT("Unit"), FText::FromString( FString( "B" ) ) );
		return FText::Format( NSLOCTEXT("Internationalization", "ComputerMemoryFormatting", "{Number} {Unit}"), Args );
	}
	while (NumBytes >= (1000000))
	{
		NumBytes /= Unit;
		++Prefix;
	}

	const double MemorySizeAsDouble = (double)NumBytes / (double)Unit;
	Args.Add( TEXT("Number"), FText::AsNumber( MemorySizeAsDouble, Options, TargetCulture) );
	Args.Add( TEXT("Unit"), FText::FromString( FString::ConstructFromPtrSize( &Prefixes[Prefix], 1 ) + Suffix) );
	return FText::Format( NSLOCTEXT("Internationalization", "ComputerMemoryFormatting", "{Number} {Unit}"), Args);
}

FText FText::AsMemory(uint64 NumBytes, EMemoryUnitStandard UnitStandard)
{
	return FText::AsMemory(NumBytes, nullptr, nullptr, UnitStandard);
}

FText FText::AsMemory(uint64 NumBytes, EMemoryUnit Unit, const FNumberFormattingOptions* const Options, const FCulturePtr& TargetCulture)
{
	if (Unit == EMemoryUnit::Auto_IEC || Unit == EMemoryUnit::Auto_SI)
	{
		return FText::AsMemory(NumBytes, Options, TargetCulture, Unit == EMemoryUnit::Auto_IEC ? EMemoryUnitStandard::IEC : EMemoryUnitStandard::SI);
	}

	checkf(FInternationalization::Get().IsInitialized() == true, TEXT("FInternationalization is not initialized. An FText formatting method was likely used in static object initialization - this is not supported."));

	if (Unit == EMemoryUnit::Unit_Bytes)
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("Number"), FText::AsNumber(NumBytes, Options, TargetCulture));
		Args.Add(TEXT("Unit"), FText::FromString(TEXT("B")));
		return FText::Format(NSLOCTEXT("Internationalization", "ComputerMemoryFormatting", "{Number} {Unit}"), Args);
	}

	static_assert((int32)EMemoryUnit::Unit_EiB - (int32)EMemoryUnit::Unit_KiB == 5, "IEC units must be contiguous in ascending order from Unit_KiB to Unit_EiB");
	static_assert((int32)EMemoryUnit::Unit_EB - (int32)EMemoryUnit::Unit_KB == 5, "SI units must be contiguous in ascending order from Unit_KB to Unit_EB");
	static_assert((int32)EMemoryUnit::Auto_SI > (int32)EMemoryUnit::Unit_EiB, "Auto_SI must be ordered after all IEC units for the bIEC range check to be correct");
	static_assert((int32)EMemoryUnit::Unit_KB > (int32)EMemoryUnit::Auto_SI, "SI units must be ordered after Auto_SI for the offset calculation to be correct");

	static const TCHAR* IECPrefixes = TEXT("KMGTPE");
	static const TCHAR* SIPrefixes = TEXT("kMGTPE");

	const bool bIEC = (Unit >= EMemoryUnit::Unit_KiB && Unit <= EMemoryUnit::Unit_EiB);
	const int32 PrefixIndex = bIEC
		? (int32)Unit - (int32)EMemoryUnit::Unit_KiB
		: (int32)Unit - (int32)EMemoryUnit::Unit_KB;
	const uint64 UnitDivisor = bIEC ? 1024ull : 1000ull;
	const TCHAR* Prefixes = bIEC ? IECPrefixes : SIPrefixes;
	const TCHAR* Suffix = bIEC ? TEXT("iB") : TEXT("B");

	uint64 Divisor = 1;
	for (int32 i = 0; i <= PrefixIndex; ++i)
	{
		Divisor *= UnitDivisor;
	}

	const double Value = (double)NumBytes / (double)Divisor;
	FFormatNamedArguments Args;
	Args.Add(TEXT("Number"), FText::AsNumber(Value, Options, TargetCulture));
	Args.Add(TEXT("Unit"), FText::FromString(FString::ConstructFromPtrSize(&Prefixes[PrefixIndex], 1) + Suffix));
	return FText::Format(NSLOCTEXT("Internationalization", "ComputerMemoryFormatting", "{Number} {Unit}"), Args);
}

FString FText::GetInvariantTimeZone()
{
	return TEXT("Etc/Unknown");
}

void FText::SerializeText(FArchive& Ar, FText& Value)
{
	SerializeText(FStructuredArchiveFromArchive(Ar).GetSlot(), Value);
}

void FText::SerializeText(FStructuredArchive::FSlot Slot, FText& Value)
{
	FArchive& UnderlyingArchive = Slot.GetUnderlyingArchive();
	FStructuredArchive::FRecord Record = Slot.EnterRecord();

	UnderlyingArchive.UsingCustomVersion(FEditorObjectVersion::GUID);

	//When duplicating, the CDO is used as the template, then values for the instance are assigned.
	//If we don't duplicate the string, the CDO and the instance are both pointing at the same thing.
	//This would result in all subsequently duplicated objects stamping over formerly duplicated ones.

	// Older FText's stored their "SourceString", that is now stored in a history class so move it there
	if (UnderlyingArchive.IsLoading() && UnderlyingArchive.UEVer() < VER_UE4_FTEXT_HISTORY)
	{
		FString SourceStringToImplantIntoHistory;
		Record << SA_VALUE(TEXT("SourceStringToImplantIntoHistory"), SourceStringToImplantIntoHistory);

		// Namespaces and keys are no longer stored in the FText, we need to read them in and discard
		FTextId TextId;
		if (UnderlyingArchive.UEVer() >= VER_UE4_ADDED_NAMESPACE_AND_KEY_DATA_TO_FTEXT)
		{
			FTextKey Namespace;
			Namespace.SerializeAsString(Record.EnterField(TEXT("Namespace")));

			FTextKey Key;
			Key.SerializeAsString(Record.EnterField(TEXT("Key")));

			// Get the DisplayString using the namespace, key, and source string.
			TextId = FTextId(Namespace, Key);
		}

		Value.TextData = MakeRefCount<FTextHistory_Base>(TextId, MoveTemp(SourceStringToImplantIntoHistory));
	}

#if WITH_EDITOR
	if (UnderlyingArchive.IsCooking() && UnderlyingArchive.IsSaving() && UnderlyingArchive.IsPersistent() && (UnderlyingArchive.GetDebugSerializationFlags() & DSF_EnableCookerWarnings))
	{
		if (!!(Value.Flags & ETextFlag::ConvertedProperty))
		{
			UE_LOGF(LogText, Warning, "Saving FText \"%ls\" which has been converted at load time please resave source package %ls to avoid determinisitic cook and localization issues.", *Value.ToString(), *UnderlyingArchive.GetArchiveName());
		}
		else if (!!(Value.Flags & ETextFlag::InitializedFromString))
		{
			UE_LOGF(LogText, Warning, "Saving FText \"%ls\" which has been initialized from FString at cook time resave of source package %ls may fix issue.", *Value.ToString(), *UnderlyingArchive.GetArchiveName())
		}
	}
#endif

	const int32 OriginalFlags = Value.Flags;

	if(UnderlyingArchive.IsSaving())
	{
		if(UnderlyingArchive.IsPersistent())
		{
			Value.Flags &= ~(ETextFlag::ConvertedProperty | ETextFlag::InitializedFromString); // Remove conversion flag before saving.
		}
	}
	Record << SA_VALUE(TEXT("Flags"), Value.Flags);

	if (UnderlyingArchive.IsLoading() && UnderlyingArchive.IsPersistent())
	{
		Value.Flags &= ~(ETextFlag::ConvertedProperty | ETextFlag::InitializedFromString); // Remove conversion flag before saving.
	}

	if (UnderlyingArchive.IsSaving())
	{
		Value.Flags = OriginalFlags;
	}

	if (UnderlyingArchive.UEVer() >= VER_UE4_FTEXT_HISTORY)
	{
		bool bSerializeHistory = true;

		if (UnderlyingArchive.IsSaving())
		{
			// Skip the history for empty texts
			bSerializeHistory = !Value.IsEmpty() && !Value.IsCultureInvariant();

			if (bSerializeHistory)
			{
				int8 HistoryType = (int8)Value.GetTextData().GetTextHistory().GetType();
				Record << SA_VALUE(TEXT("HistoryType"), HistoryType);
			}
			else
			{
				int8 HistoryType = (int8)ETextHistoryType::None;
				Record << SA_VALUE(TEXT("HistoryType"), HistoryType);

				bool bHasCultureInvariantString = !Value.IsEmpty() && Value.IsCultureInvariant();
				Record << SA_VALUE(TEXT("bHasCultureInvariantString"), bHasCultureInvariantString);
				if (bHasCultureInvariantString)
				{
					FString CultureInvariantString = Value.GetTextData().GetSourceString();
					Record << SA_VALUE(TEXT("CultureInvariantString"), CultureInvariantString);
				}
			}
		}
		else if (UnderlyingArchive.IsLoading())
		{
			int8 HistoryType = (int8)ETextHistoryType::None;
			Record << SA_VALUE(TEXT("HistoryType"), HistoryType);

			// Create the history class based on the serialized type
			switch((ETextHistoryType)HistoryType)
			{
			case ETextHistoryType::Base:
				{
					Value.TextData = MakeRefCount<FTextHistory_Base>();
					break;
				}
			case ETextHistoryType::NamedFormat:
				{
					Value.TextData = MakeRefCount<FTextHistory_NamedFormat>();
					break;
				}
			case ETextHistoryType::OrderedFormat:
				{
					Value.TextData = MakeRefCount<FTextHistory_OrderedFormat>();
					break;
				}
			case ETextHistoryType::ArgumentFormat:
				{
					Value.TextData = MakeRefCount<FTextHistory_ArgumentDataFormat>();
					break;
				}
			case ETextHistoryType::AsNumber:
				{
					Value.TextData = MakeRefCount<FTextHistory_AsNumber>();
					break;
				}
			case ETextHistoryType::AsPercent:
				{
					Value.TextData = MakeRefCount<FTextHistory_AsPercent>();
					break;
				}
			case ETextHistoryType::AsCurrency:
				{
					Value.TextData = MakeRefCount<FTextHistory_AsCurrency>();
					break;
				}
			case ETextHistoryType::AsDate:
				{
					Value.TextData = MakeRefCount<FTextHistory_AsDate>();
					break;
				}
			case ETextHistoryType::AsTime:
				{
					Value.TextData = MakeRefCount<FTextHistory_AsTime>();
					break;
				}
			case ETextHistoryType::AsDateTime:
				{
					Value.TextData = MakeRefCount<FTextHistory_AsDateTime>();
					break;
				}
			case ETextHistoryType::Transform:
				{
					Value.TextData = MakeRefCount<FTextHistory_Transform>();
					break;
				}
			case ETextHistoryType::StringTableEntry:
				{
					Value.TextData = MakeRefCount<FTextHistory_StringTableEntry>();
					break;
				}
			case ETextHistoryType::TextGenerator:
				{
					Value.TextData = MakeRefCount<FTextHistory_TextGenerator>();
					break;
				}
			default:
				{
					bSerializeHistory = false;
					Value.TextData = FText::GetEmpty().TextData;

					if (UnderlyingArchive.CustomVer(FEditorObjectVersion::GUID) >= FEditorObjectVersion::CultureInvariantTextSerializationKeyStability)
					{
						bool bHasCultureInvariantString = false;
						Record << SA_VALUE(TEXT("bHasCultureInvariantString"), bHasCultureInvariantString);
						if (bHasCultureInvariantString)
						{
							FString CultureInvariantString;
							Record << SA_VALUE(TEXT("CultureInvariantString"), CultureInvariantString);
							Value.TextData = FText(MoveTemp(CultureInvariantString)).TextData;
						}
					}
				}
			}
		}

		if (bSerializeHistory)
		{
			check(Value.TextData);
			Value.TextData->GetMutableTextHistory().Serialize(Record);
		}
	}

	if(UnderlyingArchive.IsLoading())
	{
		Value.Rebuild();
	}

	// Validate
	//if( Ar.IsLoading() && Ar.IsPersistent() && !Value.Key.IsValid() )
	//{
	//	UE_LOGF( LogText, Error, "Loaded an FText from a persistent archive but lacking a key (Namespace:%ls, Source:%ls).", Value.Namespace.Get() ? **Value.Namespace : TEXT(""), Value.SourceString.Get() ? **Value.SourceString : TEXT("") );
	//}

	if( Value.ShouldGatherForLocalization() )
	{
		UnderlyingArchive.ThisRequiresLocalizationGather();
	}
}

#if WITH_EDITORONLY_DATA
FText FText::ChangeKey( const FTextKey& Namespace, const FTextKey& Key, const FText& Text )
{
	return FText(FString(*Text.GetTextData().GetSourceString()), Namespace, Key, 0);
}

FText FText::ChangeKey( const FTextKey& Namespace, const FTextKey& Key, const FText& Text, const FString& Notes )
{
	return FText(FString(*Text.GetTextData().GetSourceString()), Namespace, Key, Notes, 0);
}

const FString& FText::GetDevNotes() const
{
	return GetTextData().GetTextHistory().GetDevNotes();
}
#endif

FText FText::FromStringTable(const FName InTableId, const FTextKey& InKey, const EStringTableLoadingPolicy InLoadingPolicy)
{
	return FStringTableRegistry::Get().Internal_FindLocTableEntry(InTableId, InKey, InLoadingPolicy);
}

FText FText::FromName( const FName& Val) 
{
	return FText::FromString(Val.ToString());
}

FText FText::FromString( const ANSICHAR* String )
{
	return FText::FromString(FString(String));
}

FText FText::FromString( const UTF8CHAR* String )
{
	return FText::FromString(FString(String));
}

FText FText::FromString( const WIDECHAR* String )
{
	return FText::FromString(FString(String));
}

FText FText::FromString( const FAnsiString& String )
{
	return FText::FromString(FString::ConstructFromPtrSize(*String, String.Len()));
}

FText FText::FromString( const FString& String )
{
	FText NewText = String.IsEmpty() ? FText::GetEmpty() : FText(CopyTemp(String));

	if (!GIsEditor)
	{
		NewText.Flags |= ETextFlag::CultureInvariant;
	}
	NewText.Flags |= ETextFlag::InitializedFromString;

	return NewText;
}

FText FText::FromString( FString&& String )
{
	FText NewText = String.IsEmpty() ? FText::GetEmpty() : FText(MoveTemp(String));

	if (!GIsEditor)
	{
		NewText.Flags |= ETextFlag::CultureInvariant;
	}
	NewText.Flags |= ETextFlag::InitializedFromString;

	return NewText;
}

FText FText::FromString( const FUtf8String& String )
{
	return FText::FromString(FString::ConstructFromPtrSize(*String, String.Len()));
}

FText FText::FromStringView(FAnsiStringView InString)
{
	FText NewText = InString.IsEmpty() ? FText::GetEmpty() : FText(FString(InString));

	if (!GIsEditor)
	{
		NewText.Flags |= ETextFlag::CultureInvariant;
	}
	NewText.Flags |= ETextFlag::InitializedFromString;

	return NewText;
}

FText FText::FromStringView(FUtf8StringView InString)
{
	FText NewText = InString.IsEmpty() ? FText::GetEmpty() : FText(FString(InString));

	if (!GIsEditor)
	{
		NewText.Flags |= ETextFlag::CultureInvariant;
	}
	NewText.Flags |= ETextFlag::InitializedFromString;

	return NewText;
}

FText FText::FromStringView(FWideStringView InString)
{
	FText NewText = InString.IsEmpty() ? FText::GetEmpty() : FText(FString(InString));

	if (!GIsEditor)
	{
		NewText.Flags |= ETextFlag::CultureInvariant;
	}
	NewText.Flags |= ETextFlag::InitializedFromString;

	return NewText;
}

FText FText::AsCultureInvariant( const ANSICHAR* String )
{
	return FText::AsCultureInvariant(FAnsiStringView(String));
}

FText FText::AsCultureInvariant( const UTF8CHAR* String )
{
	return FText::AsCultureInvariant(FUtf8StringView(String));
}

FText FText::AsCultureInvariant( const WIDECHAR* String )
{
	return FText::AsCultureInvariant(FWideStringView(String));
}

FText FText::AsCultureInvariant( FAnsiStringView String )
{
	FText NewText = String.IsEmpty() ? FText::GetEmpty() : FText(FString(String));
	NewText.Flags |= ETextFlag::CultureInvariant;

	return NewText;
}

FText FText::AsCultureInvariant( FUtf8StringView String )
{
	FText NewText = String.IsEmpty() ? FText::GetEmpty() : FText(FString(String));
	NewText.Flags |= ETextFlag::CultureInvariant;

	return NewText;
}

FText FText::AsCultureInvariant( FWideStringView String )
{
	FText NewText = String.IsEmpty() ? FText::GetEmpty() : FText(FString(String));
	NewText.Flags |= ETextFlag::CultureInvariant;

	return NewText;
}

FText FText::AsCultureInvariant( FString&& String )
{
	FText NewText = String.IsEmpty() ? FText::GetEmpty() : FText(MoveTemp(String));
	NewText.Flags |= ETextFlag::CultureInvariant;

	return NewText;
}

FText FText::AsCultureInvariant( FText Text )
{
	FText NewText = FText(MoveTemp(Text));
	NewText.Flags |= ETextFlag::CultureInvariant;

	return NewText;
}

#if WITH_EDITORONLY_DATA
FText FText::AsLocalizable_Advanced(const FTextKey& Namespace, const FTextKey& Key, const TCHAR* String, FStringView InDevNotes)
{
	return FText::AsLocalizable_Advanced(Namespace, Key, FStringView(String), InDevNotes);
}

FText FText::AsLocalizable_Advanced(const FTextKey& Namespace, const FTextKey& Key, FStringView String, FStringView InDevNotes)
{
	return String.IsEmpty() 
		? FText::GetEmpty() 
		: FTextCache::Get().FindOrCache(String, FTextId(Namespace, Key), InDevNotes);
}

FText FText::AsLocalizable_Advanced(const FTextKey& Namespace, const FTextKey& Key, FString&& String, FStringView InDevNotes)
{
	return String.IsEmpty() 
		? FText::GetEmpty() 
		: FTextCache::Get().FindOrCache(MoveTemp(String), FTextId(Namespace, Key), InDevNotes);
}

FText FText::AsLocalizable_Advanced_LocText(const TCHAR* Namespace, const TCHAR* Key, const TCHAR* String, FStringView InDevNotes)
{
	return FText::AsLocalizable_Advanced(Namespace, Key, FStringView(String), InDevNotes);
}
#else
FText FText::AsLocalizable_Advanced(const FTextKey& Namespace, const FTextKey& Key, const TCHAR* String)
{
	return FText::AsLocalizable_Advanced(Namespace, Key, FStringView(String));
}

FText FText::AsLocalizable_Advanced(const FTextKey& Namespace, const FTextKey& Key, FStringView String)
{
	return String.IsEmpty() 
		? FText::GetEmpty() 
		: FTextCache::Get().FindOrCache(String, FTextId(Namespace, Key));
}

FText FText::AsLocalizable_Advanced(const FTextKey& Namespace, const FTextKey& Key, FString&& String)
{
	return String.IsEmpty() 
		? FText::GetEmpty() 
		: FTextCache::Get().FindOrCache(MoveTemp(String), FTextId(Namespace, Key));
}

FText FText::AsLocalizable_Advanced_LocText(const TCHAR* Namespace, const TCHAR* Key, const TCHAR* String)
{
	return FText::AsLocalizable_Advanced(Namespace, Key, FStringView(String));
}
#endif

bool FText::FindTextInLiveTable_Advanced(const FTextKey& Namespace, const FTextKey& Key, FText& OutText, const FString* const SourceString)
{
	FTextConstDisplayStringPtr FoundString = FTextLocalizationManager::Get().FindDisplayString(Namespace, Key, SourceString);

	if (FoundString.IsValid())
	{
		OutText = FText(MakeRefCount<FTextHistory_Base>(FTextId(Namespace, Key), SourceString ? FString(*SourceString) : FString(), FoundString.ToSharedRef()));
		return true;
	}

	return false;
}

const FString& FText::ToString() const
{
	Rebuild();
	return GetTextData().GetDisplayString();
}

FString FText::BuildSourceString() const
{
	return GetTextData().GetTextHistory().BuildInvariantDisplayString();
}

bool FText::IsNumeric() const
{
	return ToString().IsNumeric();
}

int32 FText::CompareTo(const FText& Other, const ETextComparisonLevel::Type ComparisonLevel) const
{
	return FTextComparison::CompareTo(ToString(), Other.ToString(), ComparisonLevel);
}

int32 FText::CompareToCaseIgnored(const FText& Other) const
{
	return FTextComparison::CompareToCaseIgnored(ToString(), Other.ToString());
}

bool FText::EqualTo(const FText& Other, const ETextComparisonLevel::Type ComparisonLevel) const
{
	return FTextComparison::EqualTo(ToString(), Other.ToString(), ComparisonLevel);
}

bool FText::EqualToCaseIgnored(const FText& Other) const
{
	return FTextComparison::EqualToCaseIgnored(ToString(), Other.ToString());
}

void FText::Rebuild() const
{
	if (TextData)
	{
		TextData->GetMutableTextHistory().UpdateDisplayStringIfOutOfDate();
	}
}

bool FText::IsTransient() const
{
	return (Flags & ETextFlag::Transient) != 0;
}

bool FText::IsCultureInvariant() const
{
	return (Flags & ETextFlag::CultureInvariant) != 0;
}

bool FText::IsInitializedFromString() const
{
	return (Flags & ETextFlag::InitializedFromString) != 0;
}

bool FText::IsFromStringTable() const
{
	return GetTextData().GetTextHistory().GetType() == ETextHistoryType::StringTableEntry;
}

bool FText::ShouldGatherForLocalization() const
{
	if ( ! FPlatformProcess::SupportsMultithreading() )
	{
		return false;
	}

	return !(Flags & (ETextFlag::CultureInvariant | ETextFlag::Transient))
		&& !IsFromStringTable() 
		&& FText::ShouldGatherForLocalization(GetTextData().GetSourceString());
}

bool FText::ShouldGatherForLocalization(FStringView SourceString)
{
	// Don't gather empty strings
	if (SourceString.IsEmpty())
	{
		return false;
	}

	// Don't gather strings that are whitespace only
	for (const TCHAR Char : SourceString)
	{
		if (!FTextChar::IsWhitespace(Char))
		{
			return true;
		}
	}
	return false;
}

void FText::PreloadStringTable( FName InTableId, const FTextKey& InKey )
{
	FTextHistory_StringTableEntry(InTableId, InKey, EStringTableLoadingPolicy::FindOrLoad);
}

void FText::GetHistoricFormatData(TArray<FHistoricTextFormatData>& OutHistoricFormatData) const
{
	GetTextData().GetTextHistory().GetHistoricFormatData(*this, OutHistoricFormatData);
}

bool FText::GetHistoricNumericData(FHistoricTextNumericData& OutHistoricNumericData) const
{
	return GetTextData().GetTextHistory().GetHistoricNumericData(*this, OutHistoricNumericData);
}

bool FText::IdenticalTo( const FText& Other, const ETextIdenticalModeFlags CompareModeFlags ) const
{
	// If both instances point to the same data, then both instances are considered identical.
	if (TextData == Other.TextData)
	{
		return true;
	}

	Rebuild();
	Other.Rebuild();

	// If both instances point to the same localized string, then both instances are considered identical.
	// This is fast as it skips a lexical compare, however it can also return false for two instances that have identical strings, but in different pointers.
	// For instance, this method will return false for two FText objects created from FText::FromString("Wooble") as they each have unique (or null), non-shared instances.
	{
		FTextConstDisplayStringPtr DisplayStringPtr = GetTextData().GetLocalizedString();
		FTextConstDisplayStringPtr OtherDisplayStringPtr = Other.GetTextData().GetLocalizedString();
		if (DisplayStringPtr && OtherDisplayStringPtr && DisplayStringPtr == OtherDisplayStringPtr)
		{
			return true;
		}
	}

	if (EnumHasAnyFlags(CompareModeFlags, ETextIdenticalModeFlags::DeepCompare))
	{
		const FTextHistory& ThisTextHistory = GetTextData().GetTextHistory();
		const FTextHistory& OtherTextHistory = Other.GetTextData().GetTextHistory();
		if (ThisTextHistory.GetType() == OtherTextHistory.GetType() && ThisTextHistory.IdenticalTo(OtherTextHistory, CompareModeFlags))
		{
			return true;
		}
	}

	if (EnumHasAnyFlags(CompareModeFlags, ETextIdenticalModeFlags::LexicalCompareInvariants))
	{
		const bool bThisIsInvariant = (Flags & (ETextFlag::CultureInvariant | ETextFlag::InitializedFromString)) != 0;
		const bool bOtherIsInvariant = (Other.Flags & (ETextFlag::CultureInvariant | ETextFlag::InitializedFromString)) != 0;
		if (bThisIsInvariant && bOtherIsInvariant && ToString().Equals(Other.ToString(), ESearchCase::CaseSensitive))
		{
			return true;
		}
	}

	return false;
}

void operator<<(FStructuredArchive::FSlot Slot, FFormatArgumentValue& Value)
{
	FStructuredArchive::FRecord Record = Slot.EnterRecord();
	int8 TypeAsInt8 = (int8)Value.GetType();
	Record << SA_VALUE(TEXT("Type"), TypeAsInt8);
	Value.Type = (EFormatArgumentType::Type)TypeAsInt8;

	switch(Value.Type)
	{
	case EFormatArgumentType::Double:
		{
			Record << SA_VALUE(TEXT("Value"), Value.DoubleValue);
			break;
		}
	case EFormatArgumentType::Float:
		{
			Record << SA_VALUE(TEXT("Value"), Value.FloatValue);
			break;
		}
	case EFormatArgumentType::Int:
		{
			Record << SA_VALUE(TEXT("Value"), Value.IntValue);
			break;
		}
	case EFormatArgumentType::UInt:
	case EFormatArgumentType::Gender: // Gender is stored as a UInt
		{
			Record << SA_VALUE(TEXT("Value"), Value.UIntValue);
			break;
		}
	case EFormatArgumentType::Text:
		{
			if(Slot.GetArchiveState().IsLoading())
			{
				Value.TextValue = FText();
			}
			Record << SA_VALUE(TEXT("Value"), Value.TextValue.GetValue());
			break;
		}
	}
}

bool FFormatArgumentValue::IdenticalTo(const FFormatArgumentValue& Other, const ETextIdenticalModeFlags CompareModeFlags) const
{
	if (Type == Other.Type)
	{
		switch (Type)
		{
		case EFormatArgumentType::Int:
			return IntValue == Other.IntValue;
		case EFormatArgumentType::UInt:
			return UIntValue == Other.UIntValue;
		case EFormatArgumentType::Float:
			return FloatValue == Other.FloatValue;
		case EFormatArgumentType::Double:
			return DoubleValue == Other.DoubleValue;
		case EFormatArgumentType::Text:
			return GetTextValue().IdenticalTo(Other.GetTextValue(), CompareModeFlags);
		case EFormatArgumentType::Gender:
			return GetGenderValue() == Other.GetGenderValue();
		default:
			break;
		}
	}

	return false;
}


FFormatArgumentValue::FFormatArgumentValue(const class FCbValue& Value)
{
	switch (Value.GetType())
	{
	case ECbFieldType::String:
		Type = EFormatArgumentType::Text;
		TextValue = FText::FromString(FString(Value.AsString()));
		break;
	case ECbFieldType::IntegerPositive:
		Type = EFormatArgumentType::UInt;
		UIntValue = Value.AsIntegerPositive();
		break;
	case ECbFieldType::IntegerNegative:
		Type = EFormatArgumentType::Int;
		UIntValue = Value.AsIntegerNegative();
		break;
	case ECbFieldType::Float32:
		Type = EFormatArgumentType::Float;
		FloatValue = Value.AsFloat32();
		break;
	case ECbFieldType::Float64:
		Type = EFormatArgumentType::Double;
		DoubleValue = Value.AsFloat64();
		break;
	case ECbFieldType::BoolFalse:
		Type = EFormatArgumentType::Int;
		IntValue = 0;
		break;
	case ECbFieldType::BoolTrue:
		Type = EFormatArgumentType::Int;
		IntValue = 1;
		break;
	default:
		Type = EFormatArgumentType::Text;
		TextValue = FText::GetEmpty();
	}

}

FString FFormatArgumentValue::ToFormattedString(const bool bInRebuildText, const bool bInRebuildAsSource) const
{
	FString Result;
	ToFormattedString(bInRebuildText, bInRebuildAsSource, Result);
	return Result;
}

void FFormatArgumentValue::ToFormattedString(const bool bInRebuildText, const bool bInRebuildAsSource, FString& OutResult) const
{
	if (Type == EFormatArgumentType::Text)
	{
		const FText& LocalText = GetTextValue();

		// When doing a rebuild, all FText arguments need to be rebuilt during the Format
		if (bInRebuildText)
		{
			LocalText.Rebuild();
		}

		OutResult += (bInRebuildAsSource) ? LocalText.BuildSourceString() : LocalText.ToString();
	}
	else if (Type == EFormatArgumentType::Gender)
	{
		// Nothing to do
	}
	else
	{
		FInternationalization& I18N = FInternationalization::Get();
		checkf(I18N.IsInitialized() == true, TEXT("FInternationalization is not initialized. An FText formatting method was likely used in static object initialization - this is not supported."));
		const FCulture& Culture = *I18N.GetCurrentLocale();

		const FDecimalNumberFormattingRules& FormattingRules = Culture.GetDecimalNumberFormattingRules();
		const FNumberFormattingOptions& FormattingOptions = FormattingRules.CultureDefaultFormattingOptions;

		switch (Type)
		{
		case EFormatArgumentType::Int:
			FastDecimalFormat::NumberToString(IntValue, FormattingRules, FormattingOptions, OutResult);
			break;
		case EFormatArgumentType::UInt:
			FastDecimalFormat::NumberToString(UIntValue, FormattingRules, FormattingOptions, OutResult);
			break;
		case EFormatArgumentType::Float:
			FastDecimalFormat::NumberToString(FloatValue, FormattingRules, FormattingOptions, OutResult);
			break;
		case EFormatArgumentType::Double:
			FastDecimalFormat::NumberToString(DoubleValue, FormattingRules, FormattingOptions, OutResult);
			break;
		default:
			break;
		}
	}
}

FString FFormatArgumentValue::ToExportedString(const bool bStripPackageNamespace) const
{
	FString Result;
	ToExportedString(Result, bStripPackageNamespace);
	return Result;
}

void FFormatArgumentValue::ToExportedString(FString& OutResult, const bool bStripPackageNamespace) const
{
	switch (Type)
	{
	case EFormatArgumentType::Int:
		OutResult += LexToString(IntValue);
		break;
	case EFormatArgumentType::UInt:
		OutResult += LexToString(UIntValue);
		OutResult += TEXT('u');
		break;
	case EFormatArgumentType::Float:
		OutResult += LexToString(FloatValue);
		OutResult += TEXT('f');
		break;
	case EFormatArgumentType::Double:
		OutResult += LexToString(DoubleValue);
		break;
	case EFormatArgumentType::Text:
		FTextStringHelper::WriteToBuffer(OutResult, GetTextValue(), /*bRequiresQuotes*/true, bStripPackageNamespace);
		break;
	case EFormatArgumentType::Gender:
		TextStringificationUtil::WriteScopedEnumToBuffer(OutResult, TEXT("ETextGender::"), GetGenderValue());
		break;
	default:
		break;
	}
}

const TCHAR* FFormatArgumentValue::FromExportedString(const TCHAR* Buffer)
{
	// Is this a text gender?
	{
		static const FString TextGenderMarker = "ETextGender::";

		ETextGender LocalGender = ETextGender::Masculine;
		const TCHAR* Result = TextStringificationUtil::ReadScopedEnumFromBuffer(Buffer, TextGenderMarker, LocalGender);
		if (Result)
		{
			// Apply the result to us
			Type = EFormatArgumentType::Gender;
			UIntValue = (uint64)LocalGender;

			return Result;
		}
	}

	// Is this a number?
	{
		const TCHAR* Result = TextStringificationUtil::ReadNumberFromBuffer(Buffer, *this);
		if (Result)
		{
			return Result;
		}
	}

	// Fallback to processing as text
	{
		FText LocalText;
		TEXT_STRINGIFICATION_FUNC_MODIFY_BUFFER_AND_VALIDATE(FTextStringHelper::ReadFromBuffer, LocalText, nullptr, nullptr, true);

		// Apply the result to us
		Type = EFormatArgumentType::Text;
		TextValue = MoveTemp(LocalText);

		return Buffer;
	}
}

void FFormatArgumentData::ResetValue()
{
	ArgumentValueType = EFormatArgumentType::Text;
	ArgumentValue = FText::GetEmpty();
	ArgumentValueInt = 0;
	ArgumentValueFloat = 0.0f;
	ArgumentValueDouble = 0.0;
	ArgumentValueGender = ETextGender::Masculine;
}

FFormatArgumentValue FFormatArgumentData::ToArgumentValue() const
{
	switch (ArgumentValueType)
	{
	case EFormatArgumentType::Int:
		return FFormatArgumentValue(ArgumentValueInt);
	case EFormatArgumentType::Float:
		return FFormatArgumentValue(ArgumentValueFloat);
	case EFormatArgumentType::Double:
		return FFormatArgumentValue(ArgumentValueDouble);
	case EFormatArgumentType::Text:
		return FFormatArgumentValue(ArgumentValue);
	case EFormatArgumentType::Gender:
		return FFormatArgumentValue(ArgumentValueGender);
	default:
		break;
	}
	return FFormatArgumentValue();
}

void operator<<(FStructuredArchive::FSlot Slot, FFormatArgumentData& Value)
{
	FArchive& UnderlyingArchive = Slot.GetUnderlyingArchive();
	FStructuredArchive::FRecord Record = Slot.EnterRecord();

	UnderlyingArchive.UsingCustomVersion(FEditorObjectVersion::GUID);
	UnderlyingArchive.UsingCustomVersion(FUE5ReleaseStreamObjectVersion::GUID);
	
	if (UnderlyingArchive.IsLoading())
	{
		// ArgumentName was changed to be FString rather than FText, so we need to convert older data to ensure serialization stays happy outside of UStruct::SerializeTaggedProperties.
		if (UnderlyingArchive.UEVer() >= VER_UE4_K2NODE_VAR_REFERENCEGUIDS) // There was no version bump for this change, but VER_UE4_K2NODE_VAR_REFERENCEGUIDS was made at almost the same time.
		{
			Record << SA_VALUE(TEXT("ArgumentName"), Value.ArgumentName);
		}
		else
		{
			FText TempValue;
			Record << SA_VALUE(TEXT("ArgumentName"), TempValue);
			Value.ArgumentName = TempValue.ToString();
		}
	}
	if (UnderlyingArchive.IsSaving())
	{
		Record << SA_VALUE(TEXT("ArgumentName"), Value.ArgumentName);
	}

	uint8 TypeAsByte = (uint8)Value.ArgumentValueType;
	if (UnderlyingArchive.IsLoading())
	{
		Value.ResetValue();

		if (UnderlyingArchive.CustomVer(FEditorObjectVersion::GUID) >= FEditorObjectVersion::TextFormatArgumentDataIsVariant)
		{
			Record << SA_VALUE(TEXT("Type"), TypeAsByte);
		}
		else
		{
			// Old data was always text
			TypeAsByte = EFormatArgumentType::Text;
		}
	}
	else if (UnderlyingArchive.IsSaving())
	{
		Record << SA_VALUE(TEXT("Type"), TypeAsByte);
	}

	Value.ArgumentValueType = (EFormatArgumentType::Type)TypeAsByte;
	switch (Value.ArgumentValueType)
	{
	case EFormatArgumentType::Int:
		if (UnderlyingArchive.CustomVer(FUE5ReleaseStreamObjectVersion::GUID) >= FUE5ReleaseStreamObjectVersion::TextFormatArgumentData64bitSupport)
		{
			Record << SA_VALUE(TEXT("Value"), Value.ArgumentValueInt);	
		}
		// Legacy support for binary serialization of int32 values before 64 bit support was the default
		else
		{
			int32 IntValue = static_cast<int32>(Value.ArgumentValueInt);
			Record << SA_VALUE(TEXT("Value"), IntValue);
			Value.ArgumentValueInt = static_cast<int64>(IntValue);
		}
		break;
	case EFormatArgumentType::Float:
		Record << SA_VALUE(TEXT("Value"), Value.ArgumentValueFloat);
		break;
	case EFormatArgumentType::Double:
		Record << SA_VALUE(TEXT("Value"), Value.ArgumentValueDouble);
		break;
	case EFormatArgumentType::Text:
		Record << SA_VALUE(TEXT("Value"), Value.ArgumentValue);
		break;
	case EFormatArgumentType::Gender:
		{
			uint8& Gender = (uint8&)Value.ArgumentValueGender;
			Record << SA_VALUE(TEXT("Value"), Gender);
			break;
		}
	default:
		break;
	}
}

FTextSnapshot::FTextSnapshot()
{
}

FTextSnapshot::FTextSnapshot(const FText& InText)
	: TextDataPtr(InText.TextData)
	, LocalizedStringPtr(InText.GetTextData().GetLocalizedString())
	, GlobalHistoryRevision(GetGlobalHistoryRevisionForText(InText))
	, LocalHistoryRevision(GetLocalHistoryRevisionForText(InText))
	, Flags(InText.Flags)
{
}

bool FTextSnapshot::IdenticalTo(const FText& InText) const
{
	// Make sure the string is up-to-date with the current culture
	// (this usually happens when ToString() is called)
	InText.Rebuild();

	return TextDataPtr
		&& TextDataPtr == InText.TextData
		&& LocalizedStringPtr == InText.GetTextData().GetLocalizedString()
		&& GlobalHistoryRevision == GetGlobalHistoryRevisionForText(InText)
		&& LocalHistoryRevision == GetLocalHistoryRevisionForText(InText)
		&& Flags == InText.Flags;
}

bool FTextSnapshot::IsDisplayStringEqualTo(const FText& InText) const
{
	// Make sure the string is up-to-date with the current culture
	// (this usually happens when ToString() is called)
	InText.Rebuild();

	// We have to assume that the display string has changed if the history of the text has changed
	// (due to a culture change), as we no longer have the old display string to compare against
	return TextDataPtr
		&& GlobalHistoryRevision == GetGlobalHistoryRevisionForText(InText)
		&& LocalHistoryRevision == GetLocalHistoryRevisionForText(InText)
		&& TextDataPtr->GetDisplayString().Equals(InText.GetTextData().GetDisplayString(), ESearchCase::CaseSensitive);
}

uint16 FTextSnapshot::GetGlobalHistoryRevisionForText(const FText& InText)
{
	return (InText.IsEmpty() || InText.IsCultureInvariant()) ? 0 : InText.GetTextData().GetGlobalHistoryRevision();
}

uint16 FTextSnapshot::GetLocalHistoryRevisionForText(const FText& InText)
{
	return (InText.IsEmpty() || InText.IsCultureInvariant()) ? 0 : InText.GetTextData().GetLocalHistoryRevision();
}

bool TextBiDi::IsControlCharacter(const TCHAR InChar)
{
	return InChar == TEXT('\u061C')  // ARABIC LETTER MARK
		|| InChar == TEXT('\u200E')  // LEFT-TO-RIGHT MARK
		|| InChar == TEXT('\u200F')  // RIGHT-TO-LEFT MARK
		|| InChar == TEXT('\u202A')  // LEFT-TO-RIGHT EMBEDDING
		|| InChar == TEXT('\u202B')  // RIGHT-TO-LEFT EMBEDDING
		|| InChar == TEXT('\u202C')  // POP DIRECTIONAL FORMATTING
		|| InChar == TEXT('\u202D')  // LEFT-TO-RIGHT OVERRIDE
		|| InChar == TEXT('\u202E')  // RIGHT-TO-LEFT OVERRIDE
		|| InChar == TEXT('\u2066')  // LEFT-TO-RIGHT ISOLATE
		|| InChar == TEXT('\u2067')  // RIGHT-TO-LEFT ISOLATE
		|| InChar == TEXT('\u2068')  // FIRST STRONG ISOLATE
		|| InChar == TEXT('\u2069'); // POP DIRECTIONAL ISOLATE
}

FText FTextStringHelper::CreateFromBuffer(const TCHAR* Buffer, const TCHAR* TextNamespace, const TCHAR* PackageNamespace, const bool bRequiresQuotes)
{
	FText Value;
	if (!ReadFromBuffer(Buffer, Value, TextNamespace, PackageNamespace, bRequiresQuotes))
	{
		Value = FText::FromString(Buffer);
	}
	return Value;
}

const TCHAR* FTextStringHelper::ReadFromBuffer_ComplexText(const TCHAR* Buffer, FText& OutValue, const TCHAR* TextNamespace, const TCHAR* PackageNamespace)
{
	// Culture invariant text?
	if (TEXT_STRINGIFICATION_PEEK_MARKER(TextStringificationUtil::InvTextMarker))
	{
#define LOC_DEFINE_REGION
		// Parsing something of the form: INVTEXT("...")
		TEXT_STRINGIFICATION_SKIP_MARKER_LEN(TextStringificationUtil::InvTextMarker);

		// Skip whitespace before the opening bracket, and then step over it
		TEXT_STRINGIFICATION_SKIP_WHITESPACE_AND_CHAR('(');

		// Skip whitespace before the value, and then read out the quoted string
		FString InvariantString;
		TEXT_STRINGIFICATION_SKIP_WHITESPACE();
		TEXT_STRINGIFICATION_READ_QUOTED_STRING(InvariantString);

		// Skip whitespace before the closing bracket, and then step over it
		TEXT_STRINGIFICATION_SKIP_WHITESPACE_AND_CHAR(')');

		OutValue = FText::AsCultureInvariant(MoveTemp(InvariantString));

		return Buffer;
#undef LOC_DEFINE_REGION
	}

	// Is this text that should be parsed via its text history?
	{
		auto CreateTextHistory = [&](FText& InOutTmpText) -> bool
		{
			#define CONDITIONAL_CREATE_TEXT_HISTORY(HistoryClass)			\
				if (HistoryClass::StaticShouldReadFromBuffer(Buffer))		\
				{															\
					InOutTmpText.TextData = MakeRefCount<HistoryClass>();	\
					return true;											\
				}
			CONDITIONAL_CREATE_TEXT_HISTORY(FTextHistory_Base);
			CONDITIONAL_CREATE_TEXT_HISTORY(FTextHistory_NamedFormat);
			CONDITIONAL_CREATE_TEXT_HISTORY(FTextHistory_OrderedFormat);
			CONDITIONAL_CREATE_TEXT_HISTORY(FTextHistory_ArgumentDataFormat);
			CONDITIONAL_CREATE_TEXT_HISTORY(FTextHistory_AsNumber);
			CONDITIONAL_CREATE_TEXT_HISTORY(FTextHistory_AsPercent);
			CONDITIONAL_CREATE_TEXT_HISTORY(FTextHistory_AsCurrency);
			CONDITIONAL_CREATE_TEXT_HISTORY(FTextHistory_AsDateTime);
			CONDITIONAL_CREATE_TEXT_HISTORY(FTextHistory_AsDate);
			CONDITIONAL_CREATE_TEXT_HISTORY(FTextHistory_AsTime);
			CONDITIONAL_CREATE_TEXT_HISTORY(FTextHistory_Transform);
			CONDITIONAL_CREATE_TEXT_HISTORY(FTextHistory_StringTableEntry);
			#undef CONDITIONAL_CREATE_TEXT_HISTORY
			return false;
		};

		FText TmpText;
		if (CreateTextHistory(TmpText))
		{
			// Read the string into the text history
			TEXT_STRINGIFICATION_FUNC_MODIFY_BUFFER_AND_VALIDATE(TmpText.TextData->GetMutableTextHistory().ReadFromBuffer, TextNamespace, PackageNamespace);
			
			// Rebuild the text if we parsed its history correctly
			TmpText.Rebuild();
				
			// Move out temporary into the result
			OutValue = MoveTemp(TmpText);

			return Buffer;
		}
	}

	return nullptr;
}

const TCHAR* FTextStringHelper::ReadFromBuffer(const TCHAR* Buffer, FText& OutValue, const TCHAR* TextNamespace, const TCHAR* PackageNamespace, const bool bRequiresQuotes)
{
	check(Buffer);

	// Empty buffer?
	if (*Buffer == 0)
	{
		if (bRequiresQuotes)
		{
			return nullptr;
		}

		OutValue = FText::GetEmpty();
		return Buffer;
	}

	// First, try and parse the text as a complex text export
	{
		const TCHAR* Result = ReadFromBuffer_ComplexText(Buffer, OutValue, TextNamespace, PackageNamespace);
		if (Result)
		{
			Buffer = Result;
			return Buffer;
		}
	}

	// Quoted string?
	if (bRequiresQuotes)
	{
		// Parse out the quoted source string
		FString LiteralString;

		int32 SubNumCharsRead = 0;
		if (FParse::QuotedString(Buffer, LiteralString, &SubNumCharsRead))
		{
			OutValue = FText::FromString(MoveTemp(LiteralString));
			Buffer += SubNumCharsRead;
			return Buffer;
		}

		return nullptr;
	}

	// Raw string
	{
		FString LiteralString = Buffer;
		Buffer += LiteralString.Len(); // Advance the buffer by the length of the string to indicate success for things like ImportText
		OutValue = FText::FromString(MoveTemp(LiteralString));
		return Buffer;
	}
}

void FTextStringHelper::WriteToBuffer(FString& Buffer, const FText& Value, const bool bRequiresQuotes, const bool bStripPackageNamespace)
{
	// Culture invariant text?
	if (Value.IsCultureInvariant())
	{
#define LOC_DEFINE_REGION
		// Produces INVTEXT("...")
		Buffer += TEXT("INVTEXT(\"");
		Buffer += Value.ToString().ReplaceCharWithEscapedChar();
		Buffer += TEXT("\")");
#undef LOC_DEFINE_REGION
	}
	// Is this text that should be written via its text history?
	else if (Value.GetTextData().GetTextHistory().WriteToBuffer(Buffer, bStripPackageNamespace))
	{
	}
	// This isn't special text, so write as a raw string (potentially quoted)
	else if (bRequiresQuotes)
	{
		Buffer += TEXT("\"");
		Buffer += Value.ToString().ReplaceCharWithEscapedChar();
		Buffer += TEXT("\"");
	}
	else
	{
		Buffer += Value.ToString();
	}
}

bool FTextStringHelper::IsComplexText(const TCHAR* Buffer)
{
#define LOC_DEFINE_REGION
	return TEXT_STRINGIFICATION_PEEK_MARKER(TextStringificationUtil::InvTextMarker)
		|| FTextHistory_Base::StaticShouldReadFromBuffer(Buffer)
		|| FTextHistory_NamedFormat::StaticShouldReadFromBuffer(Buffer)
		|| FTextHistory_OrderedFormat::StaticShouldReadFromBuffer(Buffer)
		|| FTextHistory_ArgumentDataFormat::StaticShouldReadFromBuffer(Buffer)
		|| FTextHistory_AsNumber::StaticShouldReadFromBuffer(Buffer)
		|| FTextHistory_AsPercent::StaticShouldReadFromBuffer(Buffer)
		|| FTextHistory_AsCurrency::StaticShouldReadFromBuffer(Buffer)
		|| FTextHistory_AsDateTime::StaticShouldReadFromBuffer(Buffer)
		|| FTextHistory_AsDate::StaticShouldReadFromBuffer(Buffer)
		|| FTextHistory_AsTime::StaticShouldReadFromBuffer(Buffer)
		|| FTextHistory_Transform::StaticShouldReadFromBuffer(Buffer)
		|| FTextHistory_StringTableEntry::StaticShouldReadFromBuffer(Buffer);
#undef LOC_DEFINE_REGION
}

void FTextBuilder::Indent()
{
	++IndentCount;
}

void FTextBuilder::Unindent()
{
	--IndentCount;
}

void FTextBuilder::AppendLine()
{
	BuildAndAppendLine(FText());
}

void FTextBuilder::AppendLine(const FText& Text)
{
	BuildAndAppendLine(CopyTemp(Text));
}

void FTextBuilder::AppendLine(const FString& String)
{
	BuildAndAppendLine(CopyTemp(String));
}

void FTextBuilder::AppendLine(const FName& Name)
{
	BuildAndAppendLine(Name.ToString());
}

void FTextBuilder::AppendLineFormat(const FTextFormat& Pattern, const FFormatNamedArguments& Arguments)
{
	BuildAndAppendLine(FText::Format(Pattern, Arguments));
}

void FTextBuilder::AppendLineFormat(const FTextFormat& Pattern, const FFormatOrderedArguments& Arguments)
{
	BuildAndAppendLine(FText::Format(Pattern, Arguments));
}

void FTextBuilder::Clear()
{
	Lines.Reset();
}

bool FTextBuilder::IsEmpty() const
{
	return Lines.Num() == 0;
}

int32 FTextBuilder::GetNumLines() const
{
	return Lines.Num();
}

FText FTextBuilder::ToText() const
{
	return FText::Join(FText::AsCultureInvariant(LINE_TERMINATOR), Lines);
}

void FTextBuilder::BuildAndAppendLine(FString&& Data)
{
	if (IndentCount <= 0)
	{
		Lines.Emplace(FText::AsCultureInvariant(MoveTemp(Data)));
	}
	else
	{
		FString IndentedLine;
		IndentedLine.Reserve((4 * IndentCount) + Data.Len());
		for (int32 Index = 0; Index < IndentCount; ++Index)
		{
			IndentedLine += TEXT("    ");
		}
		IndentedLine += Data;
		Lines.Emplace(FText::AsCultureInvariant(MoveTemp(IndentedLine)));
	}
}

void FTextBuilder::BuildAndAppendLine(FText&& Data)
{
	if (IndentCount <= 0)
	{
		Lines.Emplace(MoveTemp(Data));
	}
	else
	{
		FString IndentedFmtPattern;
		IndentedFmtPattern.Reserve((4 * IndentCount) + 3);
		for (int32 Index = 0; Index < IndentCount; ++Index)
		{
			IndentedFmtPattern += TEXT("    ");
		}
		IndentedFmtPattern += TEXT("{0}");
		Lines.Emplace(FText::Format(FText::AsCultureInvariant(MoveTemp(IndentedFmtPattern)), MoveTemp(Data)));
	}
}

UTF32CHAR FTextChar::GetCodepoint(const TCHAR* Buffer, int32* OutNumTCHARInCodepoint)
{
	static_assert(sizeof(TCHAR) == 2 || sizeof(TCHAR) == 4, "Expected TCHAR to be UTF-16 or UTF-32");

	if (!Buffer || !Buffer[0])
	{
		return 0;
	}

	UTF32CHAR Codepoint = 0;

	const TCHAR NextChar = *(Buffer + 1);
	if (NextChar && StringConv::IsHighSurrogate(*Buffer) && StringConv::IsLowSurrogate(NextChar))
	{
		Codepoint = StringConv::EncodeSurrogate(*Buffer, NextChar);
		if (OutNumTCHARInCodepoint)
		{
			*OutNumTCHARInCodepoint = 2;
		}
	}
	else
	{
		Codepoint = *Buffer;
		if (OutNumTCHARInCodepoint)
		{
			*OutNumTCHARInCodepoint = 1;
		}
	}

	return Codepoint;
}

bool FTextChar::ConvertCodepointToString(const UTF32CHAR Codepoint, FString& OutString)
{
	OutString.Reset();
	return FTextChar::AppendCodepointToString(Codepoint, OutString);
}

bool FTextChar::AppendCodepointToString(const UTF32CHAR Codepoint, FString& OutString)
{
	static_assert(sizeof(TCHAR) == 2 || sizeof(TCHAR) == 4, "Expected TCHAR to be UTF-16 or UTF-32");

	if (!StringConv::IsValidCodepoint(Codepoint))
	{
		return false;
	}
	
	if (StringConv::IsHighSurrogate(Codepoint) || StringConv::IsLowSurrogate(Codepoint)) // Surrogate pairs range should not be present in UTF-32
	{
		return false;
	}

	if (StringConv::IsEncodedSurrogate(Codepoint))
	{
		uint16 HighSurrogate = 0;
		uint16 LowSurrogate = 0;
		StringConv::DecodeSurrogate(Codepoint, HighSurrogate, LowSurrogate);

		OutString.AppendChar((TCHAR)HighSurrogate);
		OutString.AppendChar((TCHAR)LowSurrogate);
	}
	else if (Codepoint > ENCODED_SURROGATE_END_CODEPOINT)
	{
		// Ignore values higher than the supplementary plane range
		return false;
	}
	else
	{
		OutString.AppendChar((TCHAR)Codepoint);
	}

	return true;
}

bool FUnicodeChar::CodepointToString(const uint32 InCodepoint, FString& OutString)
{
	return FTextChar::ConvertCodepointToString(InCodepoint, OutString);
}

bool LexTryParseString(ETextGender& OutValue, const TCHAR* Buffer)
{
#define ENUM_CASE_FROM_STRING(Enum) if (FCString::Stricmp(Buffer, TEXT(#Enum)) == 0) { OutValue = ETextGender::Enum; return true; }
	ENUM_CASE_FROM_STRING(Masculine);
	ENUM_CASE_FROM_STRING(Feminine);
	ENUM_CASE_FROM_STRING(Neuter);
#undef ENUM_CASE_FROM_STRING
	return false;
}

void LexFromString(ETextGender& OutValue, const TCHAR* Buffer)
{
	OutValue = ETextGender::Masculine;
	LexTryParseString(OutValue, Buffer);
}

const TCHAR* LexToString(ETextGender InValue)
{
	switch (InValue)
	{
#define ENUM_CASE_TO_STRING(Enum) case ETextGender::Enum: return TEXT(#Enum)
		ENUM_CASE_TO_STRING(Masculine);
		ENUM_CASE_TO_STRING(Feminine);
		ENUM_CASE_TO_STRING(Neuter);
#undef ENUM_CASE_TO_STRING
	default:
		return TEXT("<Unknown ETextGender>");
	}
}

bool LexTryParseString(EDateTimeStyle::Type& OutValue, const TCHAR* Buffer)
{
#define ENUM_CASE_FROM_STRING(Enum) if (FCString::Stricmp(Buffer, TEXT(#Enum)) == 0) { OutValue = EDateTimeStyle::Enum; return true; }
	ENUM_CASE_FROM_STRING(Default);
	ENUM_CASE_FROM_STRING(Short);
	ENUM_CASE_FROM_STRING(Medium);
	ENUM_CASE_FROM_STRING(Long);
	ENUM_CASE_FROM_STRING(Full);
	ENUM_CASE_FROM_STRING(Custom);
#undef ENUM_CASE_FROM_STRING
	return false;
}

void LexFromString(EDateTimeStyle::Type& OutValue, const TCHAR* Buffer)
{
	OutValue = EDateTimeStyle::Default;
	LexTryParseString(OutValue, Buffer);
}

const TCHAR* LexToString(EDateTimeStyle::Type InValue)
{
	switch (InValue)
	{
#define ENUM_CASE_TO_STRING(Enum) case EDateTimeStyle::Enum: return TEXT(#Enum)
		ENUM_CASE_TO_STRING(Default);
		ENUM_CASE_TO_STRING(Short);
		ENUM_CASE_TO_STRING(Medium);
		ENUM_CASE_TO_STRING(Long);
		ENUM_CASE_TO_STRING(Full);
		ENUM_CASE_TO_STRING(Custom);
#undef ENUM_CASE_TO_STRING
	default:
		return TEXT("<Unknown EDateTimeStyle>");
	}
}

bool LexTryParseString(ERoundingMode& OutValue, const TCHAR* Buffer)
{
#define ENUM_CASE_FROM_STRING(Enum) if (FCString::Stricmp(Buffer, TEXT(#Enum)) == 0) { OutValue = ERoundingMode::Enum; return true; }
	ENUM_CASE_FROM_STRING(HalfToEven);
	ENUM_CASE_FROM_STRING(HalfFromZero);
	ENUM_CASE_FROM_STRING(HalfToZero);
	ENUM_CASE_FROM_STRING(FromZero);
	ENUM_CASE_FROM_STRING(ToZero);
	ENUM_CASE_FROM_STRING(ToNegativeInfinity);
	ENUM_CASE_FROM_STRING(ToPositiveInfinity);
#undef ENUM_CASE_FROM_STRING
	return false;
}

void LexFromString(ERoundingMode& OutValue, const TCHAR* Buffer)
{
	OutValue = ERoundingMode::HalfToEven;
	LexTryParseString(OutValue, Buffer);
}

const TCHAR* LexToString(ERoundingMode InValue)
{
	switch (InValue)
	{
#define ENUM_CASE_TO_STRING(Enum) case ERoundingMode::Enum: return TEXT(#Enum)
	ENUM_CASE_TO_STRING(HalfToEven);
	ENUM_CASE_TO_STRING(HalfFromZero);
	ENUM_CASE_TO_STRING(HalfToZero);
	ENUM_CASE_TO_STRING(FromZero);
	ENUM_CASE_TO_STRING(ToZero);
	ENUM_CASE_TO_STRING(ToNegativeInfinity);
	ENUM_CASE_TO_STRING(ToPositiveInfinity);
#undef ENUM_CASE_TO_STRING
	default:
		return TEXT("<Unknown ERoundingMode>");
	}
}

#undef LOCTEXT_NAMESPACE
