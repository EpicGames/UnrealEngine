// Copyright Epic Games, Inc. All Rights Reserved.

#include "Internationalization/TextCache.h"

#include "Containers/UnrealString.h"
#include "HAL/LowLevelMemTracker.h"
#include "Internationalization/Text.h"
#include "Internationalization/TextKey.h"
#include "Misc/CString.h"
#include "Misc/LazySingleton.h"
#include "Misc/ScopeLock.h"

FTextCache& FTextCache::Get()
{
	return TLazySingleton<FTextCache>::Get();
}

void FTextCache::TearDown()
{
	return TLazySingleton<FTextCache>::TearDown();
}

#if WITH_EDITORONLY_DATA
FText FTextCache::FindOrCache(const TCHAR* InTextLiteral, const FTextId& InTextId, FStringView InDevNotes)
{
	return FindOrCache(FStringView(InTextLiteral), InTextId, InDevNotes);
}
#else
FText FTextCache::FindOrCache(const TCHAR* InTextLiteral, const FTextId& InTextId)
{
	return FindOrCache(FStringView(InTextLiteral), InTextId);
}
#endif

#if WITH_EDITORONLY_DATA
FText FTextCache::FindOrCache(const FStringView InTextLiteral, const FTextId& InTextId, FStringView InDevNotes)
#else
FText FTextCache::FindOrCache(const FStringView InTextLiteral, const FTextId& InTextId)
#endif
{
	LLM_SCOPE(ELLMTag::Localization);

	// First try and find a cached instance
	{
		FText Text;
		bool bTextFound = false;
		CachedText.FindAndApply(InTextId,
			[&InTextLiteral, &Text, &bTextFound](const FText& FoundText)
			{
				const FString* FoundTextLiteral = FTextInspector::GetSourceString(FoundText);
				if (FoundTextLiteral && InTextLiteral.Equals(*FoundTextLiteral, ESearchCase::CaseSensitive))
				{
					Text = FoundText;
					bTextFound = true;
				}
			}
		);

		if (bTextFound)
		{
			return Text;
		}
	}

	// Not currently cached, make a new instance...
#if WITH_EDITORONLY_DATA
	FText NewText = FText(FString(InTextLiteral), InTextId.GetNamespace(), InTextId.GetKey(), FString(InDevNotes), ETextFlag::Immutable);
#else
	FText NewText = FText(FString(InTextLiteral), InTextId.GetNamespace(), InTextId.GetKey(), ETextFlag::Immutable);
#endif

	// ... and add it to the cache
	CachedText.Add(InTextId, NewText);

	return NewText;
}

#if WITH_EDITORONLY_DATA
FText FTextCache::FindOrCache(FString&& InTextLiteral, const FTextId& InTextId, FStringView InDevNotes)
#else
FText FTextCache::FindOrCache(FString&& InTextLiteral, const FTextId& InTextId)
#endif
{
	LLM_SCOPE(ELLMTag::Localization);

	// First try and find a cached instance
	{
		FText Text;
		bool bTextFound = false;
		CachedText.FindAndApply(InTextId,
			[&InTextLiteral, &Text, &bTextFound](const FText& FoundText)
			{
				const FString* FoundTextLiteral = FTextInspector::GetSourceString(FoundText);
				if (FoundTextLiteral && InTextLiteral.Equals(*FoundTextLiteral, ESearchCase::CaseSensitive))
				{
					Text = FoundText;
					bTextFound = true;
				}
			}
		);

		if (bTextFound)
		{
			return Text;
		}
	}

	// Not currently cached, make a new instance...
#if WITH_EDITORONLY_DATA
	FText NewText = FText(FString(InTextLiteral), InTextId.GetNamespace(), InTextId.GetKey(), FString(InDevNotes), ETextFlag::Immutable);
#else
	FText NewText = FText(FString(InTextLiteral), InTextId.GetNamespace(), InTextId.GetKey(), ETextFlag::Immutable);
#endif

	// ... and add it to the cache
	CachedText.Add(InTextId, NewText);

	return NewText;
}

void FTextCache::RemoveCache(const FTextId& InTextId)
{
	return RemoveCache(MakeArrayView(&InTextId, 1));
}

void FTextCache::RemoveCache(TArrayView<const FTextId> InTextIds)
{
	for (const FTextId& TextId : InTextIds)
	{
		CachedText.Remove(TextId);
	}
}

void FTextCache::RemoveCache(const TSet<FTextId>& InTextIds)
{
	for (const FTextId& TextId : InTextIds)
	{
		CachedText.Remove(TextId);
	}
}
