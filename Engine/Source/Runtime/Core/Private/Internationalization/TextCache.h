// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Containers/StripedMap.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "Misc/TransactionallySafeCriticalSection.h"

class FTextId;

/** Caches FText instances generated via the LOCTEXT macro to avoid repeated constructions */
class FTextCache
{
public:
	/** 
	 * Get the singleton instance of the text cache.
	 */
	static FTextCache& Get();
	static void TearDown();

	FTextCache() = default;

	/** Non-copyable */
	FTextCache(const FTextCache&) = delete;
	FTextCache& operator=(const FTextCache&) = delete;

	/**
	 * Try and find an existing cached entry for the given data, or construct and cache a new entry if one cannot be found.
	 */

#if WITH_EDITORONLY_DATA
	FText FindOrCache(const TCHAR* InTextLiteral, const FTextId& InTextId, FStringView InDevNotes = {});
	FText FindOrCache(FStringView InTextLiteral, const FTextId& InTextId, FStringView InDevNotes = {});
	FText FindOrCache(FString&& InTextLiteral, const FTextId& InTextId, FStringView InDevNotes = {});
#else
	FText FindOrCache(const TCHAR* InTextLiteral, const FTextId& InTextId);
	FText FindOrCache(FStringView InTextLiteral, const FTextId& InTextId);
	FText FindOrCache(FString&& InTextLiteral, const FTextId& InTextId);
#endif

	/**
	 * Remove any cached entries for the given text IDs.
	 */
	void RemoveCache(const FTextId& InTextId);
	void RemoveCache(TArrayView<const FTextId> InTextIds);
	void RemoveCache(const TSet<FTextId>& InTextIds);

private:
	TStripedMap<32, FTextId, FText, FDefaultSetAllocator, TDefaultMapHashableKeyFuncs<FTextId, FText, false>, FTransactionallySafeStripedMapLockingPolicy> CachedText;
};
