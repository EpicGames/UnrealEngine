// Copyright Epic Games, Inc. All Rights Reserved.

#include "Internationalization/TextFormatArgumentModifier.h"

#include "Containers/UnrealString.h"
#include "Internationalization/Culture.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/TextFormatter.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/CString.h"
#include "Misc/Char.h"
#include "Misc/Parse.h"
#include "Templates/Less.h"
#include "Templates/Tuple.h"
#include "Templates/UnrealTemplate.h"

bool ITextFormatArgumentModifier::ParseKeyValueArgs(const FTextFormatString& InArgsString, TSortedMap<FTextFormatString, FTextFormatString>& OutArgKeyValues, const TCHAR InValueSeparator, const TCHAR InArgSeparator)
{
	const TCHAR* BufferPtr = InArgsString.StringPtr;
	const TCHAR* BufferEnd = InArgsString.StringPtr + InArgsString.StringLen;

	auto SkipWhitespace = [&]()
	{
		while (BufferPtr < BufferEnd && FChar::IsWhitespace(*BufferPtr))
		{
			++BufferPtr;
		}
	};

	auto ParseKey = [&]() -> FTextFormatString
	{
		const TCHAR* KeyStartPtr = BufferPtr;
		while (BufferPtr < BufferEnd && FChar::IsIdentifier(*BufferPtr) && *BufferPtr != InValueSeparator)
		{
			++BufferPtr;
		}
		return FTextFormatString::MakeReference(KeyStartPtr, UE_PTRDIFF_TO_INT32(BufferPtr - KeyStartPtr));
	};

	auto ParseValue = [&]() -> FTextFormatString
	{
		if (*BufferPtr == TEXT('"'))
		{
			FString QuotedString;
			int32 NumCharsRead = 0;
			if (FParse::QuotedString(BufferPtr, QuotedString, &NumCharsRead))
			{
				BufferPtr += NumCharsRead;
				return FTextFormatString(MoveTemp(QuotedString));
			}
		}

		const TCHAR* ValueStartPtr = BufferPtr;
		while (BufferPtr < BufferEnd && *BufferPtr != InArgSeparator)
		{
			++BufferPtr;
		}
		return FTextFormatString::MakeReference(ValueStartPtr, UE_PTRDIFF_TO_INT32(BufferPtr - ValueStartPtr));
	};

	// Skip leading whitespace in case this string is all whitespace
	SkipWhitespace();

	while (BufferPtr < BufferEnd)
	{
		// Skip whitespace up-to the argument name
		SkipWhitespace();

		// Parse the argument name
		FTextFormatString Key = ParseKey();
		if (Key.StringLen == 0)
		{
			return false;
		}

		// Skip whitespace up-to the value separator
		SkipWhitespace();

		// Ensure we have a valid value separator
		if (BufferPtr < BufferEnd && *BufferPtr++ != InValueSeparator)
		{
			return false;
		}

		// Skip whitespace up-to the argument value
		SkipWhitespace();

		// Parse the argument value
		FTextFormatString Value = ParseValue();
		if (Value.StringLen == 0)
		{
			return false;
		}

		// Skip whitespace up-to the argument separator
		SkipWhitespace();

		// Ensure we have a valid argument separator, or end of the string
		if (BufferPtr < BufferEnd && *BufferPtr++ != InArgSeparator)
		{
			return false;
		}

		OutArgKeyValues.Add(MoveTemp(Key), MoveTemp(Value));
	}

	return true;
}

bool ITextFormatArgumentModifier::ParseValueArgs(const FTextFormatString& InArgsString, TArray<FTextFormatString>& OutArgValues, const TCHAR InArgSeparator)
{
	const TCHAR* BufferPtr = InArgsString.StringPtr;
	const TCHAR* BufferEnd = InArgsString.StringPtr + InArgsString.StringLen;

	auto SkipWhitespace = [&]()
	{
		while (BufferPtr < BufferEnd && FChar::IsWhitespace(*BufferPtr))
		{
			++BufferPtr;
		}
	};

	auto ParseValue = [&]() -> FTextFormatString
	{
		if (*BufferPtr == TEXT('"'))
		{
			FString QuotedString;
			int32 NumCharsRead = 0;
			if (FParse::QuotedString(BufferPtr, QuotedString, &NumCharsRead))
			{
				BufferPtr += NumCharsRead;
				return QuotedString;
			}
		}

		const TCHAR* ValueStartPtr = BufferPtr;
		while (BufferPtr < BufferEnd && *BufferPtr != InArgSeparator)
		{
			++BufferPtr;
		}
		return FTextFormatString::MakeReference(ValueStartPtr, UE_PTRDIFF_TO_INT32(BufferPtr - ValueStartPtr));
	};

	// Skip leading whitespace in case this string is all whitespace
	SkipWhitespace();

	while (BufferPtr < BufferEnd)
	{
		// Skip whitespace up-to the argument value
		SkipWhitespace();

		// Parse the argument value
		FTextFormatString Value = ParseValue();
		if (Value.StringLen == 0)
		{
			return false;
		}

		// Skip whitespace up-to the argument separator
		SkipWhitespace();

		// Ensure we have a valid argument separator, or end of the string
		if (BufferPtr < BufferEnd && *BufferPtr++ != InArgSeparator)
		{
			return false;
		}

		OutArgValues.Add(MoveTemp(Value));
	}

	return true;
}


TSharedPtr<ITextFormatArgumentModifier> FTextFormatArgumentModifier_PluralForm::Create(const ETextPluralType InPluralType, const FTextFormatString& InArgsString, const FTextFormatPatternDefinitionConstRef& InPatternDef)
{
	TSortedMap<FTextFormatString, FTextFormatString> ArgKeyValues;
	if (ParseKeyValueArgs(InArgsString, ArgKeyValues))
	{
		int32 LongestPluralFormStringLen = 0;
		bool bDoPluralFormsUseFormatArgs = false;

		// Plural forms may contain format markers, so pre-compile all the variants now so that Evaluate doesn't have to (this also lets us validate the plural form strings and fail if they're not correct)
		TSortedMap<FTextFormatString, FTextFormat> PluralForms;
		PluralForms.Reserve(ArgKeyValues.Num());
		for (const auto& Pair : ArgKeyValues)
		{
			FTextFormat PluralForm = FTextFormat::FromString(FString::ConstructFromPtrSize(Pair.Value.StringPtr, Pair.Value.StringLen), InPatternDef);
			if (!PluralForm.IsValid())
			{
				break;
			}

			LongestPluralFormStringLen = FMath::Max(LongestPluralFormStringLen, Pair.Value.StringLen);
			bDoPluralFormsUseFormatArgs |= PluralForm.GetExpressionType() == FTextFormat::EExpressionType::Complex;

			PluralForms.Add(Pair.Key, MoveTemp(PluralForm));
		}

		// Did everything compile?
		if (PluralForms.Num() == ArgKeyValues.Num())
		{
			return MakeShareable(new FTextFormatArgumentModifier_PluralForm(InPluralType, PluralForms, LongestPluralFormStringLen, bDoPluralFormsUseFormatArgs));
		}
	}

	return nullptr;
}

bool FTextFormatArgumentModifier_PluralForm::Validate(const FCultureRef& InCulture, TArray<FString>& OutValidationErrors) const
{
	auto PluralTypeToString = [](const ETextPluralType InPluralType) -> const TCHAR*
	{
		return (InPluralType == ETextPluralType::Cardinal)
			? TEXT("Cardinal")
			: TEXT("Ordinal");
	};

	auto PluralFormToString = [](const ETextPluralForm InPluralForm) -> const TCHAR*
	{
		switch (InPluralForm)
		{
		case ETextPluralForm::Zero:
			return TEXT("zero");
		case ETextPluralForm::One:
			return TEXT("one");
		case ETextPluralForm::Two:
			return TEXT("two");
		case ETextPluralForm::Few:
			return TEXT("few");
		case ETextPluralForm::Many:
			return TEXT("many");
		default:
			return TEXT("other");
		}
	};

	bool bIsValid = true;

	const TArray<ETextPluralForm> ValidPluralForms = InCulture->GetValidPluralForms(PluralType);
	if (ValidPluralForms.Num() == 1)
	{
		bIsValid = false;
		OutValidationErrors.Add(FString::Printf(TEXT("%s plural form argument modifier is redundant as this culture only has a single plural form"), PluralTypeToString(PluralType)));
		if (CompiledPluralForms[(int32)ETextPluralForm::Other].GetSourceString().IsEmpty())
		{
			OutValidationErrors.Add(FString::Printf(TEXT("%s plural form argument modifier is missing a required plural form for '%s'"), PluralTypeToString(PluralType), PluralFormToString(ETextPluralForm::Other)));
		}
	}
	else
	{
		for (int32 CompiledPluralFormIndex = 0; CompiledPluralFormIndex < (int32)ETextPluralForm::Count; ++CompiledPluralFormIndex)
		{
			const ETextPluralForm CompiledPluralFormType = (ETextPluralForm)CompiledPluralFormIndex;
			const FTextFormat& CompiledPluralForm = CompiledPluralForms[CompiledPluralFormIndex];
			const bool bPluralFormRequired = ValidPluralForms.Contains(CompiledPluralFormType);

			if (CompiledPluralForm.GetSourceString().IsEmpty())
			{
				// Missing required plural form?
				if (bPluralFormRequired)
				{
					bIsValid = false;
					OutValidationErrors.Add(FString::Printf(TEXT("%s plural form argument modifier is missing a required plural form for '%s'"), PluralTypeToString(PluralType), PluralFormToString(CompiledPluralFormType)));
				}
			}
			else
			{
				// Provided unused plural form?
				if (!bPluralFormRequired)
				{
					bIsValid = false;
					OutValidationErrors.Add(FString::Printf(TEXT("%s plural form argument modifier has an unused plural form for '%s'"), PluralTypeToString(PluralType), PluralFormToString(CompiledPluralFormType)));
				}
			}
		}
	}

	for (const FTextFormat& CompiledPluralForm : CompiledPluralForms)
	{
		bIsValid &= CompiledPluralForm.ValidatePattern(InCulture, OutValidationErrors);
	}

	return bIsValid;
}

void FTextFormatArgumentModifier_PluralForm::Evaluate(const FFormatArgumentValue& InValue, const FPrivateTextFormatArguments& InFormatArgs, FString& OutResult) const
{
	FInternationalization& I18N = FInternationalization::Get();
	const FCulture& Culture = *I18N.GetCurrentLanguage();

	auto TryGetPluralFormForArgument = [this, &Culture](const FFormatArgumentValue& InArgValue, const int32 InArgValueMultiplier, ETextPluralForm& OutArgValuePluralForm) -> bool
	{
		switch (InArgValue.GetType())
		{
		case EFormatArgumentType::Int:
			OutArgValuePluralForm = Culture.GetPluralForm(InArgValue.GetIntValue() * static_cast<int64>(InArgValueMultiplier), PluralType);
			return true;
		case EFormatArgumentType::UInt:
			OutArgValuePluralForm = Culture.GetPluralForm(InArgValue.GetUIntValue() * static_cast<uint64>(InArgValueMultiplier), PluralType);
			return true;
		case EFormatArgumentType::Float:
			OutArgValuePluralForm = Culture.GetPluralForm(InArgValue.GetFloatValue() * static_cast<float>(InArgValueMultiplier), PluralType);
			return true;
		case EFormatArgumentType::Double:
			OutArgValuePluralForm = Culture.GetPluralForm(InArgValue.GetDoubleValue() * static_cast<double>(InArgValueMultiplier), PluralType);
			return true;
		default:
			break;
		}
		return false;
	};

	ETextPluralForm ValuePluralForm = ETextPluralForm::Other;
	if (!TryGetPluralFormForArgument(InValue, 1, ValuePluralForm) && InValue.GetType() == EFormatArgumentType::Text)
	{
		const FText TextValue = InValue.GetTextValue();

		// If this text was generated from a number, extract its numeric argument and use its number to get the plural form
		FHistoricTextNumericData TextValueNumericData;
		if (FTextInspector::GetHistoricNumericData(TextValue, TextValueNumericData))
		{
			TryGetPluralFormForArgument(
				TextValueNumericData.SourceValue, 
				TextValueNumericData.FormatType == FHistoricTextNumericData::EType::AsPercent ? 100 : 1, 
				ValuePluralForm
				);
		}
	}

	OutResult += FTextFormatter::Format(CompiledPluralForms[(int32)ValuePluralForm], InFormatArgs);
}

void FTextFormatArgumentModifier_PluralForm::GetFormatArgumentNames(TArray<FString>& OutArgumentNames) const
{
	CompiledPluralForms[(int32)ETextPluralForm::Zero].GetFormatArgumentNames(OutArgumentNames);
	CompiledPluralForms[(int32)ETextPluralForm::One].GetFormatArgumentNames(OutArgumentNames);
	CompiledPluralForms[(int32)ETextPluralForm::Two].GetFormatArgumentNames(OutArgumentNames);
	CompiledPluralForms[(int32)ETextPluralForm::Few].GetFormatArgumentNames(OutArgumentNames);
	CompiledPluralForms[(int32)ETextPluralForm::Many].GetFormatArgumentNames(OutArgumentNames);
	CompiledPluralForms[(int32)ETextPluralForm::Other].GetFormatArgumentNames(OutArgumentNames);
}

void FTextFormatArgumentModifier_PluralForm::EstimateLength(int32& OutLength, bool& OutUsesFormatArgs) const
{
	OutLength = LongestPluralFormStringLen;
	OutUsesFormatArgs = bDoPluralFormsUseFormatArgs;
}

FTextFormatArgumentModifier_PluralForm::FTextFormatArgumentModifier_PluralForm(const ETextPluralType InPluralType, const TSortedMap<FTextFormatString, FTextFormat>& InPluralForms, const int32 InLongestPluralFormStringLen, const bool InDoPluralFormsUseFormatArgs)
	: PluralType(InPluralType)
	, LongestPluralFormStringLen(InLongestPluralFormStringLen)
	, bDoPluralFormsUseFormatArgs(InDoPluralFormsUseFormatArgs)
{
	static const FTextFormatString ZeroString  = FTextFormatString::MakeReference(TEXT("zero"));
	static const FTextFormatString OneString   = FTextFormatString::MakeReference(TEXT("one"));
	static const FTextFormatString TwoString   = FTextFormatString::MakeReference(TEXT("two"));
	static const FTextFormatString FewString   = FTextFormatString::MakeReference(TEXT("few"));
	static const FTextFormatString ManyString  = FTextFormatString::MakeReference(TEXT("many"));
	static const FTextFormatString OtherString = FTextFormatString::MakeReference(TEXT("other"));

	CompiledPluralForms[(int32)ETextPluralForm::Zero]  = InPluralForms.FindRef(ZeroString);
	CompiledPluralForms[(int32)ETextPluralForm::One]   = InPluralForms.FindRef(OneString);
	CompiledPluralForms[(int32)ETextPluralForm::Two]   = InPluralForms.FindRef(TwoString);
	CompiledPluralForms[(int32)ETextPluralForm::Few]   = InPluralForms.FindRef(FewString);
	CompiledPluralForms[(int32)ETextPluralForm::Many]  = InPluralForms.FindRef(ManyString);
	CompiledPluralForms[(int32)ETextPluralForm::Other] = InPluralForms.FindRef(OtherString);
}


TSharedPtr<ITextFormatArgumentModifier> FTextFormatArgumentModifier_GenderForm::Create(const FTextFormatString& InArgsString, const FTextFormatPatternDefinitionConstRef& InPatternDef)
{
	TArray<FTextFormatString> ArgValues;
	if (ParseValueArgs(InArgsString, ArgValues) && (ArgValues.Num() == 2 || ArgValues.Num() == 3))
	{
		// Gender forms may contain format markers, so pre-compile all the variants now so that Evaluate doesn't have to (this also lets us validate the gender form strings and fail if they're not correct)
		FTextFormat MasculineForm = FTextFormat::FromString(FString::ConstructFromPtrSize(ArgValues[0].StringPtr, ArgValues[0].StringLen), InPatternDef);
		FTextFormat FeminineForm  = FTextFormat::FromString(FString::ConstructFromPtrSize(ArgValues[1].StringPtr, ArgValues[1].StringLen), InPatternDef);
		FTextFormat NeuterForm;
		if (ArgValues.Num() == 3)
		{
			NeuterForm = FTextFormat::FromString(FString::ConstructFromPtrSize(ArgValues[2].StringPtr, ArgValues[2].StringLen), InPatternDef);
		}

		// Did everything compile?
		if (MasculineForm.IsValid() && FeminineForm.IsValid())
		{
			int32 LongestGenderFormStringLen = FMath::Max(ArgValues[0].StringLen, ArgValues[1].StringLen);
			if (ArgValues.Num() == 3)
			{
				LongestGenderFormStringLen = FMath::Max(LongestGenderFormStringLen, ArgValues[2].StringLen);
			}

			const bool bDoGenderFormsUseFormatArgs = MasculineForm.GetExpressionType() == FTextFormat::EExpressionType::Complex 
				|| FeminineForm.GetExpressionType() == FTextFormat::EExpressionType::Complex 
				|| NeuterForm.GetExpressionType() == FTextFormat::EExpressionType::Complex;

			return MakeShareable(new FTextFormatArgumentModifier_GenderForm(MoveTemp(MasculineForm), MoveTemp(FeminineForm), MoveTemp(NeuterForm), LongestGenderFormStringLen, bDoGenderFormsUseFormatArgs));
		}
	}

	return nullptr;
}

bool FTextFormatArgumentModifier_GenderForm::Validate(const FCultureRef& InCulture, TArray<FString>& OutValidationErrors) const
{
	bool bIsValid = true;

	bIsValid &= MasculineForm.ValidatePattern(InCulture, OutValidationErrors);
	bIsValid &= FeminineForm.ValidatePattern(InCulture, OutValidationErrors);
	bIsValid &= NeuterForm.ValidatePattern(InCulture, OutValidationErrors);

	return bIsValid;
}

void FTextFormatArgumentModifier_GenderForm::Evaluate(const FFormatArgumentValue& InValue, const FPrivateTextFormatArguments& InFormatArgs, FString& OutResult) const
{
	if (InValue.GetType() == EFormatArgumentType::Gender)
	{
		switch (InValue.GetGenderValue())
		{
		case ETextGender::Masculine:
			OutResult += FTextFormatter::Format(MasculineForm, InFormatArgs);
			break;
		case ETextGender::Feminine:
			OutResult += FTextFormatter::Format(FeminineForm, InFormatArgs);
			break;
		case ETextGender::Neuter:
			OutResult += FTextFormatter::Format(NeuterForm, InFormatArgs);
			break;
		default:
			break;
		}
	}
}

void FTextFormatArgumentModifier_GenderForm::GetFormatArgumentNames(TArray<FString>& OutArgumentNames) const
{
	MasculineForm.GetFormatArgumentNames(OutArgumentNames);
	FeminineForm.GetFormatArgumentNames(OutArgumentNames);
	NeuterForm.GetFormatArgumentNames(OutArgumentNames);
}

void FTextFormatArgumentModifier_GenderForm::EstimateLength(int32& OutLength, bool& OutUsesFormatArgs) const
{
	OutLength = LongestGenderFormStringLen;
	OutUsesFormatArgs = bDoGenderFormsUseFormatArgs;
}

FTextFormatArgumentModifier_GenderForm::FTextFormatArgumentModifier_GenderForm(FTextFormat&& InMasculineForm, FTextFormat&& InFeminineForm, FTextFormat&& InNeuterForm, const int32 InLongestGenderFormStringLen, const bool InDoGenderFormsUseFormatArgs)
	: LongestGenderFormStringLen(InLongestGenderFormStringLen)
	, bDoGenderFormsUseFormatArgs(InDoGenderFormsUseFormatArgs)
	, MasculineForm(MoveTemp(InMasculineForm))
	, FeminineForm(MoveTemp(InFeminineForm))
	, NeuterForm(MoveTemp(InNeuterForm))
{
}


TSharedPtr<ITextFormatArgumentModifier> FTextFormatArgumentModifier_HangulPostPositions::Create(const FTextFormatString& InArgsString)
{
	TArray<FTextFormatString> ArgValues;
	if (ParseValueArgs(InArgsString, ArgValues) && ArgValues.Num() == 2)
	{
		return MakeShareable(new FTextFormatArgumentModifier_HangulPostPositions(MoveTemp(ArgValues[0]), MoveTemp(ArgValues[1])));
	}

	return nullptr;
}

bool FTextFormatArgumentModifier_HangulPostPositions::Validate(const FCultureRef& InCulture, TArray<FString>& OutValidationErrors) const
{
	return true;
}

void FTextFormatArgumentModifier_HangulPostPositions::Evaluate(const FFormatArgumentValue& InValue, const FPrivateTextFormatArguments& InFormatArgs, FString& OutResult) const
{
	const int32 ArgStartPos = OutResult.Len();
	FTextFormatter::ArgumentValueToFormattedString(InValue, InFormatArgs, OutResult);
	const int32 ArgEndPos = OutResult.Len();

	if (ArgStartPos != ArgEndPos)
	{
		const TCHAR LastArgChar = OutResult[ArgEndPos - 1];

		// Only Hangul and numeric characters need to add a suffix
		if ((LastArgChar >= 0xAC00 && LastArgChar <= 0xD7A3) || (LastArgChar >= TEXT('0') && LastArgChar <= TEXT('9')))
		{
			bool bIsConsonant = ((LastArgChar - 0xAC00) % 28 != 0) || LastArgChar == TEXT('0') || LastArgChar == TEXT('1') || LastArgChar == TEXT('3') || LastArgChar == TEXT('6') || LastArgChar == TEXT('7') || LastArgChar == TEXT('8');
			if (bIsConsonant && SuffixMode == ESuffixMode::ConsonantNotRieulOrVowel)
			{
				// We shouldn't treat Rieul as a consonant when using this suffix mode (for (eu)ro))
				const bool bIsRieul = ((LastArgChar - 0xAC00) % 28 == 8) || LastArgChar == TEXT('1') || LastArgChar == TEXT('7') || LastArgChar == TEXT('8');
				if (bIsRieul)
				{
					bIsConsonant = false;
				}
			}

			// Append the correct suffix
			if (bIsConsonant)
			{
				OutResult.AppendChars(ConsonantSuffix.StringPtr, ConsonantSuffix.StringLen);
			}
			else
			{
				OutResult.AppendChars(VowelSuffix.StringPtr, VowelSuffix.StringLen);
			}
		}
	}
}

void FTextFormatArgumentModifier_HangulPostPositions::GetFormatArgumentNames(TArray<FString>& OutArgumentNames) const
{
}

void FTextFormatArgumentModifier_HangulPostPositions::EstimateLength(int32& OutLength, bool& OutUsesFormatArgs) const
{
	OutLength = 2;
	OutUsesFormatArgs = false;
}

FTextFormatArgumentModifier_HangulPostPositions::FTextFormatArgumentModifier_HangulPostPositions(FTextFormatString&& InConsonantSuffix, FTextFormatString&& InVowelSuffix)
	: ConsonantSuffix(MoveTemp(InConsonantSuffix))
	, VowelSuffix(MoveTemp(InVowelSuffix))
	, SuffixMode(ESuffixMode::ConsonantOrVowel)
{
	// We shouldn't treat Rieul as a consonant when using (eu)ro)
	if (ConsonantSuffix.StringLen == 2 && VowelSuffix.StringLen == 1 && FCString::Strncmp(ConsonantSuffix.StringPtr, TEXT("\uC73C\uB85C"), 2) == 0 && FCString::Strncmp(VowelSuffix.StringPtr, TEXT("\uB85C"), 1) == 0)
	{
		SuffixMode = ESuffixMode::ConsonantNotRieulOrVowel;
	}
}
