// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/SortedMap.h"
#include "CoreTypes.h"
#include "Internationalization/CulturePointer.h"
#include "Internationalization/ITextFormatArgumentModifier.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"

class FString;
struct FPrivateTextFormatArguments;

/**
 * Plural form argument modifier.
 * Takes a set of key->value arguments, where the key is a valid plural form identifier, and the value is an optionally quoted string that may contain format markers.
 *  eg) |plural(one=is,other=are)
 *  eg) |ordinal(one=st,two=nd,few=rd,other=th)
 */
class FTextFormatArgumentModifier_PluralForm : public ITextFormatArgumentModifier
{
public:
	static TSharedPtr<ITextFormatArgumentModifier> Create(const ETextPluralType InPluralType, const FTextFormatString& InArgsString, const FTextFormatPatternDefinitionConstRef& InPatternDef);

	virtual bool Validate(const FCultureRef& InCulture, TArray<FString>& OutValidationErrors) const override;

	virtual void Evaluate(const FFormatArgumentValue& InValue, const FPrivateTextFormatArguments& InFormatArgs, FString& OutResult) const override;

	virtual void GetFormatArgumentNames(TArray<FString>& OutArgumentNames) const override;

	virtual void EstimateLength(int32& OutLength, bool& OutUsesFormatArgs) const override;

private:
	FTextFormatArgumentModifier_PluralForm(const ETextPluralType InPluralType, const TSortedMap<FTextFormatString, FTextFormat>& InPluralForms, const int32 InLongestPluralFormStringLen, const bool InDoPluralFormsUseFormatArgs);

	ETextPluralType PluralType;
	int32 LongestPluralFormStringLen;
	bool bDoPluralFormsUseFormatArgs;
	FTextFormat CompiledPluralForms[(int32)ETextPluralForm::Count];
};

/**
 * Gender form argument modifier.
 * Takes two (or three) value arguments, where the 0th entry is the masculine version, the 1st entry is the feminine version, and the 2nd entry is an optional neuter version. The values are an optionally quoted string that may contain format markers.
 *  eg) |gender(le,la)
 */
class FTextFormatArgumentModifier_GenderForm : public ITextFormatArgumentModifier
{
public:
	static TSharedPtr<ITextFormatArgumentModifier> Create(const FTextFormatString& InArgsString, const FTextFormatPatternDefinitionConstRef& InPatternDef);

	virtual bool Validate(const FCultureRef& InCulture, TArray<FString>& OutValidationErrors) const override;

	virtual void Evaluate(const FFormatArgumentValue& InValue, const FPrivateTextFormatArguments& InFormatArgs, FString& OutResult) const override;

	virtual void GetFormatArgumentNames(TArray<FString>& OutArgumentNames) const override;

	virtual void EstimateLength(int32& OutLength, bool& OutUsesFormatArgs) const override;

private:
	FTextFormatArgumentModifier_GenderForm(FTextFormat&& InMasculineForm, FTextFormat&& InFeminineForm, FTextFormat&& InNeuterForm, const int32 InLongestGenderFormStringLen, const bool InDoGenderFormsUseFormatArgs);

	int32 LongestGenderFormStringLen;
	bool bDoGenderFormsUseFormatArgs;
	FTextFormat MasculineForm;
	FTextFormat FeminineForm;
	FTextFormat NeuterForm;
};

/**
 * Hangul Post-Positions argument modifier.
 * Takes two value arguments, where the 0th entry is the consonant version and the 1st entry is the vowel version.
 */
class FTextFormatArgumentModifier_HangulPostPositions : public ITextFormatArgumentModifier
{
public:
	static TSharedPtr<ITextFormatArgumentModifier> Create(const FTextFormatString& InArgsString);

	virtual bool Validate(const FCultureRef& InCulture, TArray<FString>& OutValidationErrors) const override;

	virtual void Evaluate(const FFormatArgumentValue& InValue, const FPrivateTextFormatArguments& InFormatArgs, FString& OutResult) const override;

	virtual void GetFormatArgumentNames(TArray<FString>& OutArgumentNames) const override;

	virtual void EstimateLength(int32& OutLength, bool& OutUsesFormatArgs) const override;

private:
	/** How to apply determine which suffix character to use */
	enum ESuffixMode
	{
		/** Any consonant should use the consonant form, and any vowel should use the vowel form */
		ConsonantOrVowel,
		/** Any consonant that isn't Rieul should use the consonant form, and any Rieul or vowel should use the vowel form */
		ConsonantNotRieulOrVowel,
	};

	FTextFormatArgumentModifier_HangulPostPositions(FTextFormatString&& InConsonantSuffix, FTextFormatString&& InVowelSuffix);

	FTextFormatString ConsonantSuffix;
	FTextFormatString VowelSuffix;
	ESuffixMode SuffixMode;
};
