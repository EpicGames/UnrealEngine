// Copyright Epic Games, Inc. All Rights Reserved.

#include "Internationalization/TextFormatter.h"

#include "CoreGlobals.h"
#include "CoreTypes.h"
#include "HAL/LowLevelMemTracker.h"
#include "HAL/PlatformProcess.h"
#include "Internationalization/CulturePointer.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/TextFormatArgumentModifier.h"
#include "Internationalization/TextHistory.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CString.h"
#include "Misc/Char.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/ExpressionParser.h"
#include "Misc/Guid.h"
#include "Misc/Optional.h"
#include "Misc/ScopeLock.h"
#include "Misc/ScopeRWLock.h"
#include "Stats/Stats.h"
#include "Templates/Less.h"
#include "Templates/Tuple.h"
#include "Templates/UnrealTemplate.h"
#include "Templates/ValueOrError.h"
#include "Trace/Detail/Channel.h"
#include "AutoRTFM.h"
#include "Async/TransactionallySafeMutex.h"

#define LOCTEXT_NAMESPACE "TextFormatter"

DECLARE_LOG_CATEGORY_EXTERN(LogTextFormatter, Log, All);
DEFINE_LOG_CATEGORY(LogTextFormatter);

DECLARE_CYCLE_STAT(TEXT("TextFormatData Compile"), STAT_TextFormatData_Compile, STATGROUP_Text);

namespace TextFormatTokens
{

/** Token representing a literal string inside the text */
struct FStringLiteral
{
	explicit FStringLiteral(const FStringToken& InString)
		: StringStartPos(InString.GetTokenStartPos())
		, StringLen(UE_PTRDIFF_TO_INT32(InString.GetTokenEndPos() - InString.GetTokenStartPos()))
	{
	}

	/** The start of the string literal */
	const TCHAR* StringStartPos;

	/** The length of the string literal */
	int32 StringLen;
};

/** Token representing a format argument */
struct FArgumentTokenSpecifier
{
	explicit FArgumentTokenSpecifier(const FStringToken& InArgument)
		: ArgumentNameStartPos(InArgument.GetTokenStartPos())
		, ArgumentNameLen(UE_PTRDIFF_TO_INT32(InArgument.GetTokenEndPos() - InArgument.GetTokenStartPos()))
		, ArgumentIndex(INDEX_NONE)
	{
		if (ArgumentNameLen > 0)
		{
			ArgumentIndex = 0;
			for (int32 NameOffset = 0; NameOffset < ArgumentNameLen; ++NameOffset)
			{
				const TCHAR C = *(ArgumentNameStartPos + NameOffset);

				if (C >= '0' && C <= '9')
				{
					ArgumentIndex *= 10;
					ArgumentIndex += C - '0';
				}
				else
				{
					ArgumentIndex = INDEX_NONE;
					break;
				}
			}
		}
	}

	/** The start of the argument name */
	const TCHAR* ArgumentNameStartPos;

	/** The length of the argument name */
	int32 ArgumentNameLen;

	/** Cached index value if the argument name is an index, or INDEX_NONE otherwise */
	int32 ArgumentIndex;
};

/** Token representing a format argument modifier */
struct FArgumentModifierTokenSpecifier
{
	FArgumentModifierTokenSpecifier(const FStringToken& InModifierPatternWithPipe, TSharedRef<ITextFormatArgumentModifier> InTextFormatArgumentModifier)
		: ModifierPatternStartPos(InModifierPatternWithPipe.GetTokenStartPos() + 1) // We don't want to store the pipe
		, ModifierPatternLen(UE_PTRDIFF_TO_INT32(InModifierPatternWithPipe.GetTokenEndPos() - InModifierPatternWithPipe.GetTokenStartPos() - 1))
		, TextFormatArgumentModifier(MoveTemp(InTextFormatArgumentModifier))
	{
	}

	/** The start of the pattern this modifier was generated from */
	const TCHAR* ModifierPatternStartPos;

	/** The length of the pattern this modifier was generated from */
	int32 ModifierPatternLen;

	/** The compiled argument modifier that should be evaluated */
	TSharedRef<ITextFormatArgumentModifier> TextFormatArgumentModifier;
};

/** Token representing an escaped character */
struct FEscapedCharacter
{
	explicit FEscapedCharacter(TCHAR InChar)
		: Character(InChar)
	{
	}

	/** The character that was escaped */
	TCHAR Character;
};

TOptional<FExpressionError> ParseArgument(const FTextFormatPatternDefinition& InPatternDef, FExpressionTokenConsumer& Consumer)
{
	// An argument token looks like {ArgName}
	FTokenStream& Stream = Consumer.GetStream();

	TOptional<FStringToken> OpeningChar = Stream.ParseSymbol(InPatternDef.ArgStartChar);
	if (!OpeningChar.IsSet())
	{
		return TOptional<FExpressionError>();
	}

	FStringToken& EntireToken = OpeningChar.GetValue();

	// Parse out the argument name
	TOptional<FStringToken> Identifier = Stream.ParseToken([&InPatternDef](TCHAR InC)
	{
		if (InC == InPatternDef.ArgEndChar)
		{
			return EParseState::StopBefore;
		}
		else
		{
			return EParseState::Continue;
		}
	}, &EntireToken);

	if (!Identifier.IsSet() || !Stream.ParseSymbol(InPatternDef.ArgEndChar, &EntireToken).IsSet())
	{
		return TOptional<FExpressionError>();
	}

	// Add the token to the consumer - this moves the read position in the stream to the end of the token
	FStringToken& IdentifierValue = Identifier.GetValue();
	Consumer.Add(EntireToken, FArgumentTokenSpecifier(IdentifierValue));
	return TOptional<FExpressionError>();
}

TOptional<FExpressionError> ParseArgumentModifier(const FTextFormatPatternDefinition& InPatternDef, FExpressionTokenConsumer& Consumer)
{
	// An argument modifier token looks like |keyword(args, ...)
	FTokenStream& Stream = Consumer.GetStream();

	TOptional<FStringToken> PipeToken = Stream.ParseSymbol(InPatternDef.ArgModChar);
	if (!PipeToken.IsSet())
	{
		return TOptional<FExpressionError>();
	}

	FStringToken& EntireToken = PipeToken.GetValue();

	// Parse out the argument modifier name
	TOptional<FStringToken> Identifier = Stream.ParseToken([](TCHAR InC)
	{
		if (InC == '(')
		{
			return EParseState::StopBefore;
		}
		else if (FChar::IsIdentifier(InC))
		{
			return EParseState::Continue;
		}
		else
		{
			return EParseState::Cancel;
		}
	}, &EntireToken);

	if (!Identifier.IsSet() || !Stream.ParseSymbol(TEXT('('), &EntireToken).IsSet())
	{
		return TOptional<FExpressionError>();
	}

	// Valid modifier name?
	FStringToken& IdentifierValue = Identifier.GetValue();
	FTextFormatter::FCompileTextArgumentModifierFuncPtr CompileTextArgumentModifierFunc = FTextFormatter::Get().FindTextArgumentModifier(FTextFormatString::MakeReference(IdentifierValue.GetTokenStartPos(), UE_PTRDIFF_TO_INT32(IdentifierValue.GetTokenEndPos() - IdentifierValue.GetTokenStartPos())));
	if (!CompileTextArgumentModifierFunc)
	{
		return TOptional<FExpressionError>();
	}

	// Parse out the argument modifier parameter text
	TOptional<FStringToken> Parameters;
	{
		TCHAR QuoteChar = TEXT('\0');
		int32 NumConsecutiveSlashes = 0;
		Parameters = Stream.ParseToken([&](TCHAR InC)
		{
			if (InC == ')' && QuoteChar == 0)
			{
				return EParseState::StopBefore;
			}
			else if (InC == '"')
			{
				if (InC == QuoteChar)
				{
					if (NumConsecutiveSlashes%2 == 0)
					{
						QuoteChar = TEXT('\0');
					}
				}
				else
				{
					QuoteChar = InC;
				}
			}
			
			if (InC == '\\')
			{
				++NumConsecutiveSlashes;
			}
			else
			{
				NumConsecutiveSlashes = 0;
			}

			return EParseState::Continue;
		}, &EntireToken);
	}

	if (!Parameters.IsSet() || !Stream.ParseSymbol(TEXT(')'), &EntireToken).IsSet())
	{
		return TOptional<FExpressionError>();
	}

	// Compile the parameters for this argument modifier
	FStringToken& ParametersValue = Parameters.GetValue();
	TSharedPtr<ITextFormatArgumentModifier> CompiledTextArgumentModifier = CompileTextArgumentModifierFunc(FTextFormatString::MakeReference(ParametersValue.GetTokenStartPos(), UE_PTRDIFF_TO_INT32(ParametersValue.GetTokenEndPos() - ParametersValue.GetTokenStartPos())), InPatternDef.AsShared());
	if (!CompiledTextArgumentModifier.IsValid())
	{
		return TOptional<FExpressionError>();
	}

	// Add the token to the consumer - this moves the read position in the stream to the end of the token
	Consumer.Add(EntireToken, FArgumentModifierTokenSpecifier(EntireToken, CompiledTextArgumentModifier.ToSharedRef()));
	return TOptional<FExpressionError>();
}

TOptional<FExpressionError> ParseEscapedChar(const FTextFormatPatternDefinition& InPatternDef, FExpressionTokenConsumer& Consumer)
{
	FTokenStream& Stream = Consumer.GetStream();

	TOptional<FStringToken> Token = Stream.ParseSymbol(InPatternDef.EscapeChar);
	if (!Token.IsSet())
	{
		return TOptional<FExpressionError>();
	}

	FStringToken& TokenValue = Token.GetValue();

	// Accumulate the next character into the token
	TOptional<FStringToken> EscapedChar = Consumer.GetStream().ParseSymbol(&TokenValue);
	if (!EscapedChar.IsSet())
	{
		return TOptional<FExpressionError>();
	}

	// Check for a valid escape character
	const TCHAR Character = *EscapedChar->GetTokenStartPos();
	if (InPatternDef.IsValidEscapeChar(Character))
	{
		// Add the token to the consumer - this moves the read position in the stream to the end of the token.
		Consumer.Add(TokenValue, FEscapedCharacter(Character));
	}

	return TOptional<FExpressionError>();
}

TOptional<FExpressionError> ParseLiteral(const FTextFormatPatternDefinition& InPatternDef, FExpressionTokenConsumer& Consumer)
{
	FTokenStream& Stream = Consumer.GetStream();

	TOptional<FStringToken> Token;
	{
		bool bFirstChar = true;
		Token = Stream.ParseToken([&](TCHAR C)
		{
			// Always include the first character, since if it was the start of a valid token then it would have been picked up by a higher priority token parser
			if (bFirstChar)
			{
				bFirstChar = false;
				return EParseState::Continue;
			}
			else if (!InPatternDef.IsLiteralBreakChar(C))
			{
				return EParseState::Continue;
			}
			else
			{
				return EParseState::StopBefore;
			}
		});
	}

	if (Token.IsSet())
	{
		// Add the token to the consumer - this moves the read position in the stream to the end of the token
		FStringToken& TokenValue = Token.GetValue();
		Consumer.Add(TokenValue, FStringLiteral(TokenValue));
	}
	return TOptional<FExpressionError>();
}

} // namespace TextFormatTokens

DEFINE_EXPRESSION_NODE_TYPE(TextFormatTokens::FStringLiteral, 0x595A123B, 0x9418491F, 0xB416E9DB, 0xD2127828)
DEFINE_EXPRESSION_NODE_TYPE(TextFormatTokens::FArgumentTokenSpecifier, 0x5FD9EF1A, 0x9D484D65, 0x92065566, 0xD3542547)
DEFINE_EXPRESSION_NODE_TYPE(TextFormatTokens::FArgumentModifierTokenSpecifier, 0x960EEAD8, 0x34D44D08, 0xBC1118D9, 0x5BDF8D43)
DEFINE_EXPRESSION_NODE_TYPE(TextFormatTokens::FEscapedCharacter, 0x460B9845, 0xAAA9420C, 0x8125F5C5, 0xE13995DF)

struct FPrivateTextFormatArguments
{
	/** Used to abstract the method of getting an argument via index or name */
	typedef TFunctionRef<const FFormatArgumentValue*(const TextFormatTokens::FArgumentTokenSpecifier&, int32)> FGetArgumentValue;

	FPrivateTextFormatArguments(FGetArgumentValue InGetArgumentValue, const int32 InEstimatedArgumentValuesLength, const bool bInRebuildText, const bool bInRebuildAsSource)
		: GetArgumentValue(InGetArgumentValue)
		, EstimatedArgumentValuesLength(InEstimatedArgumentValuesLength)
		, bRebuildText(bInRebuildText)
		, bRebuildAsSource(bInRebuildAsSource)
	{
	}

	FGetArgumentValue GetArgumentValue;
	int32 EstimatedArgumentValuesLength;
	bool bRebuildText;
	bool bRebuildAsSource;
};

class FTextFormatData
{
public:
	/**
	 * Get the common "empty" instance to use for a default constructed FTextFormat.
	 */
	static TSharedRef<FTextFormatData, ESPMode::ThreadSafe> GetSharedEmptyInstance();

	/**
	 * Construct an instance from an FText.
	 * The text will be immediately compiled. 
	 */
	FTextFormatData(FText&& InText, ETextFormatFlags InFormatFlags, FTextFormatPatternDefinitionConstRef InPatternDef);

	/**
	 * Construct an instance from an FString.
	 * The string will be immediately compiled. 
	 */
	FTextFormatData(FString&& InString, ETextFormatFlags InFormatFlags, FTextFormatPatternDefinitionConstRef InPatternDef);

	/**
	 * Test to see whether this instance contains valid compiled data.
	 */
	FORCEINLINE bool IsValid() const
	{
		UE::TScopeLock Lock(CompiledDataMutex);
		return IsValid_NoLock();
	}

	/**
	 * Check whether this instance is considered identical to the other instance, based on the comparison flags provided.
	 */
	bool IdenticalTo(const FTextFormatData& Other, const ETextIdenticalModeFlags CompareModeFlags) const;

	/**
	 * Validate the format pattern is valid based on the rules of the given culture (or null to use the current language).
	 * @return true if the pattern is valid, or false if not (false may also fill in OutValidationErrors).
	 */
	FORCEINLINE bool ValidatePattern(const FCulturePtr& InCulture, TArray<FString>& OutValidationErrors)
	{
		UE::TScopeLock Lock(CompiledDataMutex);
		return ValidatePattern_NoLock(InCulture, OutValidationErrors);
	}

	/**
	 * Produce a formatted string using the given argument look-up.
	 */
	FORCEINLINE FString Format(const FPrivateTextFormatArguments& InFormatArgs)
	{
		UE::TScopeLock Lock(CompiledDataMutex);
		return Format_NoLock(InFormatArgs);
	}

	/**
	 * Append the names of any arguments to the given array.
	 */
	FORCEINLINE void GetFormatArgumentNames(TArray<FString>& OutArgumentNames)
	{
		UE::TScopeLock Lock(CompiledDataMutex);
		GetFormatArgumentNames_NoLock(OutArgumentNames);
	}

	/**
	 * Get the source text that we're holding.
	 * If we're holding a string then we'll construct a new text.
	 */
	FORCEINLINE FText GetSourceText() const
	{
		return (SourceType == ESourceType::Text) ? SourceText : FText::FromString(SourceExpression);
	}

	/**
	 * Get the source string that we're holding.
	 * If we're holding a text then we'll return its internal string.
	 */
	FORCEINLINE const FString& GetSourceString() const
	{
		return (SourceType == ESourceType::Text) ? SourceText.ToString() : SourceExpression;
	}

	/**
	 * Get the type of expression currently compiled.
	 */
	FORCEINLINE FTextFormat::EExpressionType GetExpressionType() const
	{
		UE::TScopeLock Lock(CompiledDataMutex);
		return CompiledExpressionType;
	}

	/**
	 * Get the format flags being used.
	 */
	FORCEINLINE ETextFormatFlags GetFormatFlags() const
	{
		return FormatFlags;
	}

	/**
	 * Get the format pattern definition being used.
	 */
	FORCEINLINE FTextFormatPatternDefinitionConstRef GetPatternDefinition() const
	{
		return PatternDef;
	}

private:
	/**
	 * Test to see whether this instance contains valid compiled data.
	 * Internal version that doesn't lock, so the calling code must handle that!
	 */
	bool IsValid_NoLock() const;

	/**
	 * Compile the current state of the text into tokens that can be used for formatting.
	 * Internal version that doesn't lock, so the calling code must handle that!
	 */
	void Compile_NoLock();

	/**
	 * Compile the current state of the text into tokens that can be used for formatting, but only if the text has changed since it was last compiled.
	 * Internal version that doesn't lock, so the calling code must handle that!
	 */
	void ConditionalCompile_NoLock();

	/**
	 * Validate the format pattern is valid based on the rules of the given culture (or null to use the current language).
	 * Internal version that doesn't lock, so the calling code must handle that!
	 */
	bool ValidatePattern_NoLock(const FCulturePtr& InCulture, TArray<FString>& OutValidationErrors);

	/**
	 * Produce a formatted string using the given argument look-up.
	 * Internal version that doesn't lock, so the calling code must handle that!
	 */
	FString Format_NoLock(const FPrivateTextFormatArguments& InFormatArgs);

	/**
	 * Append the names of any arguments to the given array.
	 * Internal version that doesn't lock, so the calling code must handle that!
	 */
	void GetFormatArgumentNames_NoLock(TArray<FString>& OutArgumentNames);

	enum ESourceType : uint8
	{
		Text,
		String,
	};

	/**
	 * Type of source we're using (FText or FString).
	 */
	ESourceType SourceType;

	/**
	 * Flags controlling the behavior of the format.
	 */
	ETextFormatFlags FormatFlags;

	/**
	 * Definition of the pattern used during a text format.
	 */
	FTextFormatPatternDefinitionConstRef PatternDef;

	/**
	 * Source localized text that is used as the format specifier.
	 */
	FText SourceText;

	/**
	 * Mutex protecting the compiled data from being modified concurrently.
	 */
	mutable UE::FTransactionallySafeMutex CompiledDataMutex;

	/**
	 * Copy of the string that was last compiled.
	 * This allows the text to update via a culture change without immediately invalidating our compiled tokens.
	 * If the data was constructed from an FString rather than an FText, then this is the string we were given and shouldn't be updated once the initial construction has happened.
	 * Concurrent access protected by CompiledDataMutex.
	 */
	FString SourceExpression;

	/**
	 * Lexed expression tokens generated from, and referencing, SourceExpression.
	 * Concurrent access protected by CompiledDataMutex.
	 */
	TArray<FExpressionToken> LexedExpression;

	/**
	 * Snapshot of the text that last time it was compiled into a format expression.
	 * This is used to detect when the source text was changed and allow a re-compile.
	 * Concurrent access protected by CompiledDataMutex.
	 */
	FTextSnapshot CompiledTextSnapshot;

	/**
	 * The type of expression currently compiled.
	 * Concurrent access protected by CompiledDataMutex.
	 */
	FTextFormat::EExpressionType CompiledExpressionType;

	/**
	 * Holds the last compilation error (if any, when CompiledExpressionType == Invalid).
	 * Concurrent access protected by CompiledDataMutex.
	 */
	FString LastCompileError;

	/**
	 * The base length of the string that will go into the formatted string (no including any argument substitutions.
	 * Concurrent access protected by CompiledDataMutex.
	 */
	int32 BaseFormatStringLength;

	/**
	 * A multiplier to apply to the given argument count (base is 1, and 1 is added for every argument modifier that may make use of the arguments).
	 * Concurrent access protected by CompiledDataMutex.
	 */
	int32 FormatArgumentEstimateMultiplier;
};


FTextFormat::FTextFormat()
	: TextFormatData(FTextFormatData::GetSharedEmptyInstance())
{
}

FTextFormat::FTextFormat(const FText& InText, ETextFormatFlags InFormatFlags)
	: TextFormatData(MakeShared<FTextFormatData, ESPMode::ThreadSafe>(CopyTemp(InText), InFormatFlags, FTextFormatPatternDefinition::GetDefault()))
{
}

FTextFormat::FTextFormat(const FText& InText, FTextFormatPatternDefinitionConstRef InCustomPatternDef, ETextFormatFlags InFormatFlags)
	: TextFormatData(MakeShared<FTextFormatData, ESPMode::ThreadSafe>(CopyTemp(InText), InFormatFlags, MoveTemp(InCustomPatternDef)))
{
}

FTextFormat::FTextFormat(FString&& InString, FTextFormatPatternDefinitionConstRef InCustomPatternDef, ETextFormatFlags InFormatFlags)
	: TextFormatData(MakeShared<FTextFormatData, ESPMode::ThreadSafe>(MoveTemp(InString), InFormatFlags, MoveTemp(InCustomPatternDef)))
{
}

FTextFormat FTextFormat::FromString(const FString& InString, ETextFormatFlags InFormatFlags)
{
	return FTextFormat(CopyTemp(InString), FTextFormatPatternDefinition::GetDefault(), InFormatFlags);
}

FTextFormat FTextFormat::FromString(FString&& InString, ETextFormatFlags InFormatFlags)
{
	return FTextFormat(MoveTemp(InString), FTextFormatPatternDefinition::GetDefault(), InFormatFlags);
}

FTextFormat FTextFormat::FromString(const FString& InString, FTextFormatPatternDefinitionConstRef InCustomPatternDef, ETextFormatFlags InFormatFlags)
{
	return FTextFormat(CopyTemp(InString), MoveTemp(InCustomPatternDef), InFormatFlags);
}

FTextFormat FTextFormat::FromString(FString&& InString, FTextFormatPatternDefinitionConstRef InCustomPatternDef, ETextFormatFlags InFormatFlags)
{
	return FTextFormat(MoveTemp(InString), MoveTemp(InCustomPatternDef), InFormatFlags);
}

bool FTextFormat::IsValid() const
{
	return TextFormatData->IsValid();
}

bool FTextFormat::IdenticalTo(const FTextFormat& Other, const ETextIdenticalModeFlags CompareModeFlags) const
{
	return TextFormatData->IdenticalTo(*Other.TextFormatData, CompareModeFlags);
}

FText FTextFormat::GetSourceText() const
{
	return TextFormatData->GetSourceText();
}

const FString& FTextFormat::GetSourceString() const
{
	return TextFormatData->GetSourceString();
}

FTextFormat::EExpressionType FTextFormat::GetExpressionType() const
{
	return TextFormatData->GetExpressionType();
}

ETextFormatFlags FTextFormat::GetFormatFlags() const
{
	return TextFormatData->GetFormatFlags();
}

FTextFormatPatternDefinitionConstRef FTextFormat::GetPatternDefinition() const
{
	return TextFormatData->GetPatternDefinition();
}

bool FTextFormat::ValidatePattern(const FCulturePtr& InCulture, TArray<FString>& OutValidationErrors) const
{
	return TextFormatData->ValidatePattern(InCulture, OutValidationErrors);
}

void FTextFormat::GetFormatArgumentNames(TArray<FString>& OutArgumentNames) const
{
	TextFormatData->GetFormatArgumentNames(OutArgumentNames);
}


TSharedRef<FTextFormatData, ESPMode::ThreadSafe> FTextFormatData::GetSharedEmptyInstance()
{
	static const TSharedRef<FTextFormatData> EmptyInstance = MakeShared<FTextFormatData, ESPMode::ThreadSafe>(FText(), ETextFormatFlags::Default, FTextFormatPatternDefinition::GetDefault());
	return EmptyInstance;
}

FTextFormatData::FTextFormatData(FText&& InText, ETextFormatFlags InFormatFlags, FTextFormatPatternDefinitionConstRef InPatternDef)
	: SourceType(ESourceType::Text)
	, FormatFlags(InFormatFlags)
	, PatternDef(MoveTemp(InPatternDef))
	, SourceText(MoveTemp(InText))
{
	Compile_NoLock();
}

FTextFormatData::FTextFormatData(FString&& InString, ETextFormatFlags InFormatFlags, FTextFormatPatternDefinitionConstRef InPatternDef)
	: SourceType(ESourceType::String)
	, FormatFlags(InFormatFlags)
	, PatternDef(MoveTemp(InPatternDef))
	, SourceExpression(MoveTemp(InString))
{
	Compile_NoLock();
}

bool FTextFormatData::IdenticalTo(const FTextFormatData& Other, const ETextIdenticalModeFlags CompareModeFlags) const
{
	if (SourceType == Other.SourceType)
	{
		switch (SourceType)
		{
		case ESourceType::Text:
			return SourceText.IdenticalTo(Other.SourceText, CompareModeFlags);

		case ESourceType::String:
			if (EnumHasAnyFlags(CompareModeFlags, ETextIdenticalModeFlags::LexicalCompareInvariants))
			{
				return SourceExpression.Equals(Other.SourceExpression, ESearchCase::CaseSensitive);
			}
			break;

		default:
			break;
		}
	}

	return false;
}

bool FTextFormatData::IsValid_NoLock() const
{
	return CompiledExpressionType != FTextFormat::EExpressionType::Invalid;
}

void FTextFormatData::Compile_NoLock()
{
	SCOPE_CYCLE_COUNTER(STAT_TextFormatData_Compile);
	LLM_SCOPE(ELLMTag::Localization);

	LexedExpression.Reset();
	if (SourceType == ESourceType::Text)
	{
		SourceExpression = SourceText.ToString();
		CompiledTextSnapshot = FTextSnapshot(SourceText);
	}
	CompiledExpressionType = FTextFormat::EExpressionType::Simple;
	BaseFormatStringLength = 0;
	FormatArgumentEstimateMultiplier = 1;

	TValueOrError<TArray<FExpressionToken>, FExpressionError> Result = ExpressionParser::Lex(*SourceExpression, PatternDef->GetTextFormatDefinitions());
	bool bValidExpression = Result.IsValid();
	if (bValidExpression)
	{
		LexedExpression = Result.StealValue();

		// Quickly make sure the tokens are valid (argument modifiers may only follow an argument token)
		for (int32 TokenIndex = 0; TokenIndex < LexedExpression.Num(); ++TokenIndex)
		{
			const FExpressionToken& Token = LexedExpression[TokenIndex];

			if (const auto* Literal = Token.Node.Cast<TextFormatTokens::FStringLiteral>())
			{
				BaseFormatStringLength += Literal->StringLen;
			}
			else if (auto* Escaped = Token.Node.Cast<TextFormatTokens::FEscapedCharacter>())
			{
				BaseFormatStringLength += 1;
			}
			else if (const auto* ArgumentToken = Token.Node.Cast<TextFormatTokens::FArgumentTokenSpecifier>())
			{
				CompiledExpressionType = FTextFormat::EExpressionType::Complex;

				if (LexedExpression.IsValidIndex(TokenIndex + 1))
				{
					const FExpressionToken& NextToken = LexedExpression[TokenIndex + 1];

					// Peek to see if the next token is an argument modifier
					if (const auto* ArgumentModifierToken = NextToken.Node.Cast<TextFormatTokens::FArgumentModifierTokenSpecifier>())
					{
						if (EnumHasAnyFlags(FormatFlags, ETextFormatFlags::EvaluateArgumentModifiers))
						{
							int32 ArgModLength = 0;
							bool ArgModUsesFormatArgs = false;
							ArgumentModifierToken->TextFormatArgumentModifier->EstimateLength(ArgModLength, ArgModUsesFormatArgs);

							BaseFormatStringLength += ArgModLength;
							FormatArgumentEstimateMultiplier += (ArgModUsesFormatArgs) ? 1 : 0;
						}
						else
						{
							BaseFormatStringLength += ArgumentModifierToken->ModifierPatternLen + 1;
						}

						++TokenIndex; // walk over the argument token so that the next iteration will skip over the argument modifier
						continue;
					}
				}
			}
			else if (Token.Node.Cast<TextFormatTokens::FArgumentModifierTokenSpecifier>())
			{
				// Unexpected argument modifier token!
				const FText ErrorSourceText = FText::FromString(Token.Context.GetString());
				Result = MakeError(FExpressionError(FText::Format(LOCTEXT("UnexpectedArgumentModifierToken", "Unexpected 'argument modifier' token: {0} (token started at index {1})"), ErrorSourceText, Token.Context.GetCharacterIndex())));
				bValidExpression = false;
				break;
			}
		}
	}
	
	if (!bValidExpression)
	{
		LexedExpression.Reset();
		CompiledExpressionType = FTextFormat::EExpressionType::Invalid;
		LastCompileError = Result.GetError().Text.ToString();
	}
}

void FTextFormatData::ConditionalCompile_NoLock()
{
	// IdenticalTo compares our pointer against the static empty instance, rather than checking if our text is actually empty
	// This is what we want to happen since a text using the static empty instance will never become non-empty, but an empty string might (due to a culture change, or in-editor change)
	bool bRequiresCompile = SourceType == ESourceType::Text && !SourceText.IdenticalTo(FText::GetEmpty());

	if (bRequiresCompile)
	{
		bRequiresCompile = false;
		if (!CompiledTextSnapshot.IdenticalTo(SourceText))
		{
			if (!CompiledTextSnapshot.IsDisplayStringEqualTo(SourceText))
			{
				bRequiresCompile = true;
			}
			CompiledTextSnapshot = FTextSnapshot(SourceText); // Update this even if the text is lexically identical, as it will update the pointer compared by IdenticalTo for the next ConditionalCompile
		}
	}

	if (bRequiresCompile)
	{
		Compile_NoLock();
	}
}

bool FTextFormatData::ValidatePattern_NoLock(const FCulturePtr& InCulture, TArray<FString>& OutValidationErrors)
{
	ConditionalCompile_NoLock();

	if (CompiledExpressionType == FTextFormat::EExpressionType::Invalid)
	{
		if (!LastCompileError.IsEmpty())
		{
			OutValidationErrors.Add(LastCompileError);
		}
		return false;
	}

	if (CompiledExpressionType != FTextFormat::EExpressionType::Complex)
	{
		return true;
	}

	const FCultureRef ResolvedCulture = InCulture ? InCulture.ToSharedRef() : FInternationalization::Get().GetCurrentLanguage();

	bool bIsValidPattern = true;
	if (EnumHasAnyFlags(FormatFlags, ETextFormatFlags::EvaluateArgumentModifiers))
	{
		for (const FExpressionToken& Token : LexedExpression)
		{
			if (const auto* ArgumentModifierToken = Token.Node.Cast<TextFormatTokens::FArgumentModifierTokenSpecifier>())
			{
				bIsValidPattern &= ArgumentModifierToken->TextFormatArgumentModifier->Validate(ResolvedCulture, OutValidationErrors);
			}
		}
	}
	return bIsValidPattern;
}

FString FTextFormatData::Format_NoLock(const FPrivateTextFormatArguments& InFormatArgs)
{
	if (SourceType == ESourceType::Text && InFormatArgs.bRebuildText)
	{
		SourceText.Rebuild();
	}

	ConditionalCompile_NoLock();

	if (LexedExpression.Num() == 0)
	{
		return SourceExpression;
	}

	FString ResultString;
	ResultString.Reserve(BaseFormatStringLength + (InFormatArgs.EstimatedArgumentValuesLength * FormatArgumentEstimateMultiplier));

	int32 ArgumentIndex = 0;
	for (int32 TokenIndex = 0; TokenIndex < LexedExpression.Num(); ++TokenIndex)
	{
		const FExpressionToken& Token = LexedExpression[TokenIndex];

		if (const auto* Literal = Token.Node.Cast<TextFormatTokens::FStringLiteral>())
		{
			ResultString.AppendChars(Literal->StringStartPos, Literal->StringLen);
		}
		else if (auto* Escaped = Token.Node.Cast<TextFormatTokens::FEscapedCharacter>())
		{
			ResultString.AppendChar(Escaped->Character);
		}
		else if (const auto* ArgumentToken = Token.Node.Cast<TextFormatTokens::FArgumentTokenSpecifier>())
		{
			const FFormatArgumentValue* const PossibleArgumentValue = InFormatArgs.GetArgumentValue(*ArgumentToken, ArgumentIndex++);
			if (PossibleArgumentValue)
			{
				if (LexedExpression.IsValidIndex(TokenIndex + 1))
				{
					const FExpressionToken& NextToken = LexedExpression[TokenIndex + 1];

					// Peek to see if the next token is an argument modifier
					if (const auto* ArgumentModifierToken = NextToken.Node.Cast<TextFormatTokens::FArgumentModifierTokenSpecifier>())
					{
						if (EnumHasAnyFlags(FormatFlags, ETextFormatFlags::EvaluateArgumentModifiers))
						{
							ArgumentModifierToken->TextFormatArgumentModifier->Evaluate(*PossibleArgumentValue, InFormatArgs, ResultString);
						}
						else
						{
							// If evaluation is disabled we just write the literal value of the argument modifier back into the final string...
							ResultString.AppendChar(PatternDef->ArgModChar);
							ResultString.AppendChars(ArgumentModifierToken->ModifierPatternStartPos, ArgumentModifierToken->ModifierPatternLen);
						}
						++TokenIndex; // walk over the argument token so that the next iteration will skip over the argument modifier
						continue;
					}
				}

				PossibleArgumentValue->ToFormattedString(InFormatArgs.bRebuildText, InFormatArgs.bRebuildAsSource, ResultString);
			}
			else
			{
				ResultString.AppendChar(PatternDef->ArgStartChar);
				ResultString.AppendChars(ArgumentToken->ArgumentNameStartPos, ArgumentToken->ArgumentNameLen);
				ResultString.AppendChar(PatternDef->ArgEndChar);
			}
		}
		else if (const auto* ArgumentModifierToken = Token.Node.Cast<TextFormatTokens::FArgumentModifierTokenSpecifier>())
		{
			// If we find an argument modifier token on its own then it means an argument value failed to evaluate (likely due to InFormatArgs.GetArgumentValue returning null)
			// In this case we just write the literal value of the argument modifier back into the final string...
			ResultString.AppendChar(PatternDef->ArgModChar);
			ResultString.AppendChars(ArgumentModifierToken->ModifierPatternStartPos, ArgumentModifierToken->ModifierPatternLen);
		}
	}

	return ResultString;
}

void FTextFormatData::GetFormatArgumentNames_NoLock(TArray<FString>& OutArgumentNames)
{
	ConditionalCompile_NoLock();

	if (CompiledExpressionType != FTextFormat::EExpressionType::Complex)
	{
		return;
	}

	for (const FExpressionToken& Token : LexedExpression)
	{
		if (const auto* ArgumentToken = Token.Node.Cast<TextFormatTokens::FArgumentTokenSpecifier>())
		{
			// Add the entry to the array if it doesn't already exist
			// We can't just use AddUnique since we need the names to be case-sensitive
			const bool bIsInArray = OutArgumentNames.ContainsByPredicate([&](const FString& InEntry) -> bool
			{
				return ArgumentToken->ArgumentNameLen == InEntry.Len() && FCString::Strncmp(ArgumentToken->ArgumentNameStartPos, *InEntry, ArgumentToken->ArgumentNameLen) == 0;
			});

			if (!bIsInArray)
			{
				OutArgumentNames.Add(FString::ConstructFromPtrSize(ArgumentToken->ArgumentNameStartPos, ArgumentToken->ArgumentNameLen));
			}
		}
		else if (const auto* ArgumentModifierToken = Token.Node.Cast<TextFormatTokens::FArgumentModifierTokenSpecifier>())
		{
			if (EnumHasAnyFlags(FormatFlags, ETextFormatFlags::EvaluateArgumentModifiers))
			{
				ArgumentModifierToken->TextFormatArgumentModifier->GetFormatArgumentNames(OutArgumentNames);
			}
		}
	}
}


FTextFormatPatternDefinition::FTextFormatPatternDefinition()
{
	TextFormatDefinitions.DefineToken([this](FExpressionTokenConsumer& Consumer) { return TextFormatTokens::ParseArgument(*this, Consumer); });
	TextFormatDefinitions.DefineToken([this](FExpressionTokenConsumer& Consumer) { return TextFormatTokens::ParseArgumentModifier(*this, Consumer); });
	TextFormatDefinitions.DefineToken([this](FExpressionTokenConsumer& Consumer) { return TextFormatTokens::ParseEscapedChar(*this, Consumer); });
	TextFormatDefinitions.DefineToken([this](FExpressionTokenConsumer& Consumer) { return TextFormatTokens::ParseLiteral(*this, Consumer); });
}

FTextFormatPatternDefinitionConstRef FTextFormatPatternDefinition::GetDefault()
{
	static FTextFormatPatternDefinitionConstRef DefaultFormatPatternDefinition = MakeShared<FTextFormatPatternDefinition, ESPMode::ThreadSafe>();
	return DefaultFormatPatternDefinition;
}

const FTokenDefinitions& FTextFormatPatternDefinition::GetTextFormatDefinitions() const
{
	return TextFormatDefinitions;
}


FTextFormatter::FTextFormatter()
{
	TextArgumentModifiers.Add(FTextFormatString::MakeReference(TEXT("plural")),  [](const FTextFormatString& InArgsString, const FTextFormatPatternDefinitionConstRef& InPatternDef) { return FTextFormatArgumentModifier_PluralForm::Create(ETextPluralType::Cardinal, InArgsString, InPatternDef); });
	TextArgumentModifiers.Add(FTextFormatString::MakeReference(TEXT("ordinal")), [](const FTextFormatString& InArgsString, const FTextFormatPatternDefinitionConstRef& InPatternDef) { return FTextFormatArgumentModifier_PluralForm::Create(ETextPluralType::Ordinal, InArgsString, InPatternDef); });
	TextArgumentModifiers.Add(FTextFormatString::MakeReference(TEXT("gender")),  [](const FTextFormatString& InArgsString, const FTextFormatPatternDefinitionConstRef& InPatternDef) { return FTextFormatArgumentModifier_GenderForm::Create(InArgsString, InPatternDef); });
	TextArgumentModifiers.Add(FTextFormatString::MakeReference(TEXT("hpp")),     [](const FTextFormatString& InArgsString, const FTextFormatPatternDefinitionConstRef& InPatternDef) { return FTextFormatArgumentModifier_HangulPostPositions::Create(InArgsString); });
}

FTextFormatter& FTextFormatter::Get()
{
	static FTextFormatter TextFormatter;
	return TextFormatter;
}

void FTextFormatter::RegisterTextArgumentModifier(const FTextFormatString& InKeyword, FCompileTextArgumentModifierFuncPtr InCompileFunc)
{
	UE::TWriteScopeLock Lock(TextArgumentModifiersRW);
	TextArgumentModifiers.Add(InKeyword, MoveTemp(InCompileFunc));
}

void FTextFormatter::UnregisterTextArgumentModifier(const FTextFormatString& InKeyword)
{
	UE::TWriteScopeLock Lock(TextArgumentModifiersRW);
	TextArgumentModifiers.Remove(InKeyword);
}

FTextFormatter::FCompileTextArgumentModifierFuncPtr FTextFormatter::FindTextArgumentModifier(const FTextFormatString& InKeyword) const
{
	UE::TReadScopeLock Lock(TextArgumentModifiersRW);
	return TextArgumentModifiers.FindRef(InKeyword);
}

FText FTextFormatter::Format(FTextFormat&& InFmt, FFormatNamedArguments&& InArguments, const bool bInRebuildText, const bool bInRebuildAsSource)
{
	FString ResultString = FormatStr(InFmt, InArguments, bInRebuildText, bInRebuildAsSource);

	FText Result = FText(MakeRefCount<FTextHistory_NamedFormat>(MoveTemp(ResultString), MoveTemp(InFmt), MoveTemp(InArguments)));
	if (!GIsEditor)
	{
		Result.Flags |= ETextFlag::Transient;
	}
	return Result;
}

FText FTextFormatter::Format(FTextFormat&& InFmt, FFormatOrderedArguments&& InArguments, const bool bInRebuildText, const bool bInRebuildAsSource)
{
	FString ResultString = FormatStr(InFmt, InArguments, bInRebuildText, bInRebuildAsSource);

	FText Result = FText(MakeRefCount<FTextHistory_OrderedFormat>(MoveTemp(ResultString), MoveTemp(InFmt), MoveTemp(InArguments)));
	if (!GIsEditor)
	{
		Result.Flags |= ETextFlag::Transient;
	}
	return Result;
}

FText FTextFormatter::Format(FTextFormat&& InFmt, TArray<FFormatArgumentData>&& InArguments, const bool bInRebuildText, const bool bInRebuildAsSource)
{
	FString ResultString = FormatStr(InFmt, InArguments, bInRebuildText, bInRebuildAsSource);

	FText Result = FText(MakeRefCount<FTextHistory_ArgumentDataFormat>(MoveTemp(ResultString), MoveTemp(InFmt), MoveTemp(InArguments)));
	if (!GIsEditor)
	{
		Result.Flags |= ETextFlag::Transient;
	}
	return Result;
}

FString FTextFormatter::FormatStr(const FTextFormat& InFmt, const FFormatNamedArguments& InArguments, const bool bInRebuildText, const bool bInRebuildAsSource)
{
	if (FPlatformProcess::SupportsMultithreading())
	{
		checkf(FInternationalization::Get().IsInitialized() == true, TEXT("FInternationalization is not initialized. An FText formatting method was likely used in static object initialization - this is not supported."));
	}

	int32 EstimatedArgumentValuesLength = 0;
	for (const auto& Pair : InArguments)
	{
		EstimatedArgumentValuesLength += EstimateArgumentValueLength(Pair.Value);
	}

	auto GetArgumentValue = [&InArguments](const TextFormatTokens::FArgumentTokenSpecifier& ArgumentToken, int32 ArgumentNumber) -> const FFormatArgumentValue*
	{
		for (const auto& Pair : InArguments)
		{
			if (ArgumentToken.ArgumentNameLen == Pair.Key.Len() && FCString::Strncmp(ArgumentToken.ArgumentNameStartPos, *Pair.Key, ArgumentToken.ArgumentNameLen) == 0)
			{
				return &Pair.Value;
			}
		}
		return nullptr;
	};

	return Format(InFmt, FPrivateTextFormatArguments(FPrivateTextFormatArguments::FGetArgumentValue(GetArgumentValue), EstimatedArgumentValuesLength, bInRebuildText, bInRebuildAsSource));
}

FString FTextFormatter::FormatStr(const FTextFormat& InFmt, const FFormatOrderedArguments& InArguments, const bool bInRebuildText, const bool bInRebuildAsSource)
{
	if (FPlatformProcess::SupportsMultithreading())
	{
		checkf(FInternationalization::Get().IsInitialized() == true, TEXT("FInternationalization is not initialized. An FText formatting method was likely used in static object initialization - this is not supported."));
	}

	int32 EstimatedArgumentValuesLength = 0;
	for (const auto& Arg : InArguments)
	{
		EstimatedArgumentValuesLength += EstimateArgumentValueLength(Arg);
	}

	const FString& FmtPattern = InFmt.GetSourceString();
	auto GetArgumentValue = [&InArguments, &FmtPattern](const TextFormatTokens::FArgumentTokenSpecifier& ArgumentToken, int32 ArgumentNumber) -> const FFormatArgumentValue*
	{
		int32 ArgumentIndex = ArgumentToken.ArgumentIndex;
		if (ArgumentIndex == INDEX_NONE)
		{
			// We failed to parse the argument name into a number...
			// We have existing code that is incorrectly using names in the format string when providing ordered arguments
			// ICU used to fallback to treating the index of the argument within the string as if it were the index specified 
			// by the argument name, so we need to emulate that behavior to avoid breaking some format operations
			UE_LOGF(LogTextFormatter, Warning, "Failed to parse argument \"%ls\" as a number (using \"%d\" as a fallback). Please check your format string for errors: \"%ls\".", *FString::ConstructFromPtrSize(ArgumentToken.ArgumentNameStartPos, ArgumentToken.ArgumentNameLen), ArgumentNumber, *FmtPattern);
			ArgumentIndex = ArgumentNumber;
		}
		return InArguments.IsValidIndex(ArgumentIndex) ? &(InArguments[ArgumentIndex]) : nullptr;
	};

	return Format(InFmt, FPrivateTextFormatArguments(FPrivateTextFormatArguments::FGetArgumentValue(GetArgumentValue), EstimatedArgumentValuesLength, bInRebuildText, bInRebuildAsSource));
}

FString FTextFormatter::FormatStr(const FTextFormat& InFmt, const TArray<FFormatArgumentData>& InArguments, const bool bInRebuildText, const bool bInRebuildAsSource)
{
	if (FPlatformProcess::SupportsMultithreading())
	{
		checkf(FInternationalization::Get().IsInitialized() == true, TEXT("FInternationalization is not initialized. An FText formatting method was likely used in static object initialization - this is not supported."));
	}

	int32 EstimatedArgumentValuesLength = 0;
	for (const auto& Arg : InArguments)
	{
		EstimatedArgumentValuesLength += EstimateArgumentValueLength(Arg.ArgumentValue);
	}

	FFormatArgumentValue TmpArgumentValue;
	FFormatArgumentValue* TmpArgumentValuePtr = &TmpArgumentValue; // Need to do this to avoid the error "address of stack memory associated with local variable 'TmpArgumentValue' returned"
	auto GetArgumentValue = [&InArguments, &TmpArgumentValuePtr](const TextFormatTokens::FArgumentTokenSpecifier& ArgumentToken, int32 ArgumentNumber) -> const FFormatArgumentValue*
	{
		for (const auto& Arg : InArguments)
		{
			if (ArgumentToken.ArgumentNameLen == Arg.ArgumentName.Len() && FCString::Strncmp(ArgumentToken.ArgumentNameStartPos, *Arg.ArgumentName, ArgumentToken.ArgumentNameLen) == 0)
			{
				(*TmpArgumentValuePtr) = Arg.ToArgumentValue();
				return TmpArgumentValuePtr;
			}
		}
		return nullptr;
	};

	return Format(InFmt, FPrivateTextFormatArguments(FPrivateTextFormatArguments::FGetArgumentValue(GetArgumentValue), EstimatedArgumentValuesLength, bInRebuildText, bInRebuildAsSource));
}

FString FTextFormatter::Format(const FTextFormat& InFmt, const FPrivateTextFormatArguments& InFormatArgs)
{
	FTextFormat FmtPattern = InFmt;

	// If we're rebuilding as source then we need to handle that before we call Format
	// We don't need to worry about any rebuilding that needs to happen as non-source, as Format takes care of that internally
	if (InFormatArgs.bRebuildAsSource)
	{
		FText FmtText = InFmt.GetSourceText();

		if (InFormatArgs.bRebuildText)
		{
			FmtText.Rebuild();
		}

		FmtPattern = FTextFormat(FmtText.BuildSourceString(), FmtPattern.GetPatternDefinition(), FmtPattern.GetFormatFlags());
	}

	return FmtPattern.TextFormatData->Format(InFormatArgs);
}

void FTextFormatter::ArgumentValueToFormattedString(const FFormatArgumentValue& InValue, const FPrivateTextFormatArguments& InFormatArgs, FString& OutResult)
{
	InValue.ToFormattedString(InFormatArgs.bRebuildText, InFormatArgs.bRebuildAsSource, OutResult);
}

int32 FTextFormatter::EstimateArgumentValueLength(const FFormatArgumentValue& ArgumentValue)
{
	switch (ArgumentValue.GetType())
	{
	case EFormatArgumentType::Text:
		return ArgumentValue.GetTextValue().ToString().Len();
	case EFormatArgumentType::Int:
	case EFormatArgumentType::UInt:
	case EFormatArgumentType::Float:
	case EFormatArgumentType::Double:
		return 20;
	default:
		break;
	}

	return 0;
}

#undef LOCTEXT_NAMESPACE
