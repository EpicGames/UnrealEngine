// Copyright Epic Games, Inc. All Rights Reserved.

#include "Internationalization/TextGeneratorBase.h"

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

FName FTextGeneratorBase::GetTypeID() const
{
	return NAME_None;
}

void FTextGeneratorBase::Serialize(FStructuredArchive::FRecord Record)
{
	checkf(false, TEXT("Serialization is not supported for this text generator type"));
}
