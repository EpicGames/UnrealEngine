// Copyright Epic Games, Inc. All Rights Reserved.

#include "Internationalization/TextGeneratorRegistry.h"

#include "Misc/AssertionMacros.h"
#include "Misc/ScopeLock.h"
#include "Templates/UnrealTemplate.h"

FTextGeneratorRegistry& FTextGeneratorRegistry::Get()
{
	static FTextGeneratorRegistry Instance;
	return Instance;
}

FText::FCreateTextGeneratorDelegate FTextGeneratorRegistry::FindRegisteredTextGenerator( FName TypeID ) const
{
	FScopeLock ScopeLock(&TextGeneratorFactoryMapLock);
	const FText::FCreateTextGeneratorDelegate* Result = TextGeneratorFactoryMap.Find(TypeID);
	return Result ? *Result : FText::FCreateTextGeneratorDelegate();
}

void FTextGeneratorRegistry::RegisterTextGenerator( FName TypeID, FText::FCreateTextGeneratorDelegate FactoryFunction )
{
	FScopeLock ScopeLock(&TextGeneratorFactoryMapLock);
	check(FactoryFunction.IsBound());
	ensureAlways(!TextGeneratorFactoryMap.Contains(TypeID));
	TextGeneratorFactoryMap.Add(TypeID, MoveTemp(FactoryFunction));
}

void FTextGeneratorRegistry::UnregisterTextGenerator( FName TypeID )
{
	FScopeLock ScopeLock(&TextGeneratorFactoryMapLock);
	TextGeneratorFactoryMap.Remove(TypeID);
}
