// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Map.h"
#include "HAL/CriticalSection.h"
#include "Internationalization/Text.h"
#include "UObject/NameTypes.h"

/** Singleton registry of text generator factory functions */
class FTextGeneratorRegistry
{
public:
	/** Singleton accessor */
	static FTextGeneratorRegistry& Get();

	/**
	 * Returns the text generator factory function registered under the specified name, if any.
	 *
	 * @param TypeID the name under which to look up the factory function
	 */
	FText::FCreateTextGeneratorDelegate FindRegisteredTextGenerator( FName TypeID ) const;

	/**
	 * Registers a factory function to be used with serialization of text generators within FText.
	 *
	 * @param TypeID the name under which to register the factory function. Must match ITextGenerator::GetTypeID().
	 * @param FactoryFunction the factory function to create the generator instance
	 */
	void RegisterTextGenerator( FName TypeID, FText::FCreateTextGeneratorDelegate FactoryFunction );

	/**
	 * Unregisters a factory function to be used with serialization of text generators within FText.
	 *
	 * @param TypeID the name to remove from registration
	 *
	 * @see RegisterTextGenerator
	 */
	void UnregisterTextGenerator( FName TypeID );

private:
	/** Critical section for TextGeneratorFactoryMap */
	mutable FCriticalSection TextGeneratorFactoryMapLock;

	/** The mapping of IDs to factory functions */
	TMap<FName, FText::FCreateTextGeneratorDelegate> TextGeneratorFactoryMap;
};
