// Copyright Epic Games, Inc. All Rights Reserved.

#include "Internationalization/TextHistory.h"
#include "UObject/ObjectVersion.h"
#include "Internationalization/Culture.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/StringTableCore.h"
#include "Internationalization/StringTableRegistry.h"
#include "Misc/Guid.h"
#include "Misc/Parse.h"
#include "Misc/ScopeLock.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "UObject/PropertyPortFlags.h"

#include "Internationalization/FastDecimalFormat.h"
#include "Internationalization/ITextGenerator.h"
#include "Internationalization/TextFormatter.h"
#include "Internationalization/TextChronoFormatter.h"
#include "Internationalization/TextTransformer.h"
#include "Internationalization/TextNamespaceUtil.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"

DECLARE_LOG_CATEGORY_EXTERN(LogTextHistory, Log, All);
DEFINE_LOG_CATEGORY(LogTextHistory);

/** Utilities for stringifying text */
namespace TextStringificationUtil
{

bool PeekMarker(const TCHAR* Buffer, const TCHAR* InMarker, const int32 InMarkerLen)
{
	return FCString::Strncmp(Buffer, InMarker, InMarkerLen) == 0;
}

bool PeekInsensitiveMarker(const TCHAR* Buffer, const TCHAR* InMarker, const int32 InMarkerLen)
{
	return FCString::Strnicmp(Buffer, InMarker, InMarkerLen) == 0;
}

const TCHAR* SkipMarker(const TCHAR* Buffer, const TCHAR* InMarker, const int32 InMarkerLen)
{
	if (!PeekMarker(Buffer, InMarker, InMarkerLen))
	{
		return nullptr;
	}

	Buffer += InMarkerLen;
	return Buffer;
}

const TCHAR* SkipInsensitiveMarker(const TCHAR* Buffer, const TCHAR* InMarker, const int32 InMarkerLen)
{
	if (!PeekInsensitiveMarker(Buffer, InMarker, InMarkerLen))
	{
		return nullptr;
	}

	Buffer += InMarkerLen;
	return Buffer;
}

const TCHAR* SkipWhitespace(const TCHAR* Buffer)
{
	while (*Buffer && (*Buffer == TCHAR(' ') || *Buffer == TCHAR('\t')) && *Buffer != TCHAR('\n') && *Buffer != TCHAR('\r'))
	{
		++Buffer;
	}

	return Buffer;
}

const TCHAR* SkipWhitespaceToCharacter(const TCHAR* Buffer, const TCHAR InChar)
{
	Buffer = SkipWhitespace(Buffer);

	if (Buffer && *Buffer != InChar)
	{
		return nullptr;
	}

	return Buffer;
}

const TCHAR* SkipWhitespaceAndCharacter(const TCHAR* Buffer, const TCHAR InChar)
{
	Buffer = SkipWhitespaceToCharacter(Buffer, InChar);

	if (Buffer)
	{
		++Buffer;
	}

	return Buffer;
}

const TCHAR* ReadNumberFromBuffer(const TCHAR* Buffer, FFormatArgumentValue& OutValue)
{
	static const TCHAR* ValidNumericChars = TEXT("+-0123456789.ful");
	static const TCHAR* SuffixNumericChars = TEXT("ful");

	FString NumericString;
	while (*Buffer && FCString::Strchr(ValidNumericChars, *Buffer))
	{
		NumericString += *Buffer++;
	}

	FString SuffixString;
	while (NumericString.Len() > 0 && FCString::Strchr(SuffixNumericChars, NumericString[NumericString.Len() - 1]))
	{
		SuffixString += NumericString[NumericString.Len() - 1];
		NumericString.RemoveAt(NumericString.Len() - 1, EAllowShrinking::No);
	}

	if (!NumericString.IsNumeric())
	{
		return nullptr;
	}

	if (FCString::Strchr(*SuffixString, TEXT('f')))
	{
		// Probably a float
		float LocalFloat = 0.0f;
		LexFromString(LocalFloat, *NumericString);
		OutValue = FFormatArgumentValue(LocalFloat);
	}
	else if (FCString::Strchr(*SuffixString, TEXT('u')))
	{
		// Probably unsigned
		uint64 LocalUInt = 0;
		LexFromString(LocalUInt, *NumericString);
		OutValue = FFormatArgumentValue(LocalUInt);
	}
	else if (FCString::Strchr(*NumericString, TEXT('.')))
	{
		// Probably a double (or unmarked float)
		double LocalDouble = 0.0;
		LexFromString(LocalDouble, *NumericString);
		OutValue = FFormatArgumentValue(LocalDouble);
	}
	else
	{
		// Probably an int (or unmarked unsigned)
		int64 LocalInt = 0;
		LexFromString(LocalInt, *NumericString);
		OutValue = FFormatArgumentValue(LocalInt);
	}

	return Buffer;
}

const TCHAR* ReadAlnumFromBuffer(const TCHAR* Buffer, FString& OutValue)
{
	OutValue.Reset();
	while (*Buffer && (FChar::IsAlnum(*Buffer) || *Buffer == TEXT('_')))
	{
		OutValue += *Buffer++;
	}

	if (OutValue.IsEmpty())
	{
		return nullptr;
	}

	return Buffer;
}

const TCHAR* ReadQuotedStringFromBuffer(const TCHAR* Buffer, FString& OutStr)
{
	// Might be wrapped in TEXT(...) if the string came from C++
	const bool bIsMacroWrapped = TEXT_STRINGIFICATION_PEEK_MARKER(TextMarker);
	if (bIsMacroWrapped)
	{
		// Skip the TEXT marker
		TEXT_STRINGIFICATION_SKIP_MARKER_LEN(TextMarker);

		// Skip whitespace before the opening bracket, and then step over it
		TEXT_STRINGIFICATION_SKIP_WHITESPACE_AND_CHAR('(');
	}

	// Read the quoted string
	{
		int32 CharsRead = 0;
		if (!FParse::QuotedString(Buffer, OutStr, &CharsRead))
		{
			return nullptr;
		}
		Buffer += CharsRead;
	}

	// Skip the end of the macro
	if (bIsMacroWrapped)
	{
		// Skip whitespace before the closing bracket, and then step over it
		TEXT_STRINGIFICATION_SKIP_WHITESPACE_AND_CHAR(')');
	}

	return Buffer;
}

template <typename T>
void WriteNumberFormattingOptionToBuffer(FString& Buffer, const TCHAR* OptionFunctionName, const T& OptionValue, const T& DefaultOptionValue, TFunctionRef<void(FString&, const T&)> WriteOptionValue)
{
	if (OptionValue != DefaultOptionValue)
	{
		if (!Buffer.IsEmpty())
		{
			Buffer += TEXT('.');
		}
		Buffer += OptionFunctionName;
		Buffer += TEXT('(');
		WriteOptionValue(Buffer, OptionValue);
		Buffer += TEXT(')');
	}
}

void WriteNumberFormattingOptionsToBuffer(FString& Buffer, const FNumberFormattingOptions& Options)
{
	auto WriteBoolOption = [](FString& OutValueBuffer, const bool& InValue)
	{
		OutValueBuffer += LexToString(InValue);
	};

	auto WriteIntOption = [](FString& OutValueBuffer, const int32& InValue)
	{
		OutValueBuffer += LexToString(InValue);
	};

	auto WriteRoundingModeOption = [](FString& OutValueBuffer, const ERoundingMode& InValue)
	{
		WriteScopedEnumToBuffer(OutValueBuffer, TEXT("ERoundingMode::"), InValue);
	};

	static const FNumberFormattingOptions DefaultOptions;

	#define WRITE_CUSTOM_OPTION(Option, WriteOptionValue) WriteNumberFormattingOptionToBuffer<decltype(Options.Option)>(Buffer, TEXT("Set"#Option), Options.Option, DefaultOptions.Option, WriteOptionValue)
	WRITE_CUSTOM_OPTION(AlwaysSign, WriteBoolOption);
	WRITE_CUSTOM_OPTION(UseGrouping, WriteBoolOption);
	WRITE_CUSTOM_OPTION(RoundingMode, WriteRoundingModeOption);
	WRITE_CUSTOM_OPTION(MinimumIntegralDigits, WriteIntOption);
	WRITE_CUSTOM_OPTION(MaximumIntegralDigits, WriteIntOption);
	WRITE_CUSTOM_OPTION(MinimumFractionalDigits, WriteIntOption);
	WRITE_CUSTOM_OPTION(MaximumFractionalDigits, WriteIntOption);
	#undef WRITE_CUSTOM_OPTION
}

template <typename T>
const TCHAR* ReadNumberFormattingOptionFromBuffer(const TCHAR* Buffer, const FString& OptionFunctionName, T& OutOptionValue, TFunctionRef<const TCHAR*(const TCHAR*, T&)> ReadOptionValue)
{
	if (PeekMarker(Buffer, *OptionFunctionName, OptionFunctionName.Len()))
	{
		// Walk over the function name
		Buffer += OptionFunctionName.Len();

		// Skip whitespace before the opening bracket, and then step over it
		TEXT_STRINGIFICATION_SKIP_WHITESPACE_AND_CHAR('(');

		// Skip whitespace before the value, and then read the option value
		TEXT_STRINGIFICATION_SKIP_WHITESPACE();
		TEXT_STRINGIFICATION_FUNC_MODIFY_BUFFER_AND_VALIDATE(ReadOptionValue, OutOptionValue);

		// Skip whitespace before the closing bracket, and then step over it
		TEXT_STRINGIFICATION_SKIP_WHITESPACE_AND_CHAR(')');
	}

	return Buffer;
}

const TCHAR* ReadNumberFormattingOptionsFromBuffer(const TCHAR* Buffer, FNumberFormattingOptions& OutOptions)
{
	auto ReadBoolOption = [](const TCHAR* InValueBuffer, bool& OutValue) -> const TCHAR*
	{
		#define READ_BOOL_OPTION(Value)															\
			{																					\
				static const FString ValueString = TEXT(#Value);								\
				if (FCString::Strnicmp(InValueBuffer, *ValueString, ValueString.Len()) == 0)	\
				{																				\
					OutValue = Value;															\
					InValueBuffer += ValueString.Len();											\
					return InValueBuffer;														\
				}																				\
			}
		READ_BOOL_OPTION(true);
		READ_BOOL_OPTION(false);
		#undef READ_BOOL_OPTION 

		return nullptr;
	};

	auto ReadNumericOption = [](const TCHAR* InValueBuffer, int32& OutValue) -> const TCHAR*
	{
		FFormatArgumentValue ReadValue;
		InValueBuffer = ReadNumberFromBuffer(InValueBuffer, ReadValue);
		if (!InValueBuffer)
		{
			return nullptr;
		}

		switch (ReadValue.GetType())
		{
		case EFormatArgumentType::Int:
			OutValue = (int32)ReadValue.GetIntValue();
			break;
		case EFormatArgumentType::UInt:
			OutValue = (int32)ReadValue.GetUIntValue();
			break;
		case EFormatArgumentType::Float:
			OutValue = (int32)ReadValue.GetFloatValue();
			break;
		case EFormatArgumentType::Double:
			OutValue = (int32)ReadValue.GetDoubleValue();
			break;
		default:
			return nullptr;
		}

		return InValueBuffer;
	};

	auto ReadRoundingModeOption = [](const TCHAR* InValueBuffer, ERoundingMode& OutValue) -> const TCHAR*
	{
		static const FString RoundingModeMarker = "ERoundingMode::";
		return ReadScopedEnumFromBuffer(InValueBuffer, RoundingModeMarker, OutValue);
	};

	bool bDidReadOption = true;
	while (bDidReadOption)
	{
		bDidReadOption = false;
		#define READ_CUSTOM_OPTION(Option, ReadOptionValue)							\
		{																			\
			static const FString OptionMarker = TEXT("Set"#Option);					\
			if (*Buffer == TEXT('.')) { ++Buffer; }									\
			const TCHAR* const ValueStart = Buffer;									\
			TEXT_STRINGIFICATION_FUNC_MODIFY_BUFFER_AND_VALIDATE(ReadNumberFormattingOptionFromBuffer<decltype(OutOptions.Option)>, OptionMarker, OutOptions.Option, ReadOptionValue); \
			if (Buffer != ValueStart) { bDidReadOption = true; }					\
		}
		READ_CUSTOM_OPTION(AlwaysSign, ReadBoolOption);
		READ_CUSTOM_OPTION(UseGrouping, ReadBoolOption);
		READ_CUSTOM_OPTION(RoundingMode, ReadRoundingModeOption);
		READ_CUSTOM_OPTION(MinimumIntegralDigits, ReadNumericOption);
		READ_CUSTOM_OPTION(MaximumIntegralDigits, ReadNumericOption);
		READ_CUSTOM_OPTION(MinimumFractionalDigits, ReadNumericOption);
		READ_CUSTOM_OPTION(MaximumFractionalDigits, ReadNumericOption);
		#undef READ_CUSTOM_OPTION
	}

	return Buffer;
}

void WriteNumberOrPercentToBuffer(FString& Buffer, const TCHAR* TokenMarker, const FFormatArgumentValue& SourceValue, const TOptional<FNumberFormattingOptions>& FormatOptions, FCulturePtr TargetCulture, const bool bStripPackageNamespace)
{
	FString Suffix;
	FString CustomOptions;
	if (FormatOptions.IsSet())
	{
		if (FormatOptions->IsIdentical(FNumberFormattingOptions::DefaultWithGrouping()))
		{
			Suffix = GroupedSuffix;
		}
		else if (FormatOptions->IsIdentical(FNumberFormattingOptions::DefaultNoGrouping()))
		{
			Suffix = UngroupedSuffix;
		}
		else
		{
			WriteNumberFormattingOptionsToBuffer(CustomOptions, FormatOptions.GetValue());
			if (!CustomOptions.IsEmpty())
			{
				Suffix = CustomSuffix;
			}
		}
	}

	// Produces LOCGEN_NUMBER/_GROUPED/_UNGROUPED(..., "...") or LOCGEN_NUMBER_CUSTOM(..., ..., "...")
	// Produces LOCGEN_PERCENT/_GROUPED/_UNGROUPED(..., "...") or LOCGEN_PERCENT_CUSTOM(..., ..., "...")
	Buffer += TokenMarker;
	Buffer += Suffix;
	Buffer += TEXT("(");
	SourceValue.ToExportedString(Buffer, bStripPackageNamespace);
	if (Suffix == CustomSuffix)
	{
		Buffer += TEXT(", ");
		Buffer += CustomOptions;
	}
	Buffer += TEXT(", \"");
	if (TargetCulture)
	{
		Buffer += TargetCulture->GetName().ReplaceCharWithEscapedChar();
	}
	Buffer += TEXT("\")");
}

const TCHAR* ReadNumberOrPercentFromBuffer(const TCHAR* Buffer, const FString& TokenMarker, FFormatArgumentValue& OutSourceValue, TOptional<FNumberFormattingOptions>& OutFormatOptions, FCulturePtr& OutTargetCulture)
{
	if (PeekMarker(Buffer, *TokenMarker, TokenMarker.Len()))
	{
		// Parsing something of the form: LOCGEN_NUMBER/_GROUPED/_UNGROUPED(..., "...") or LOCGEN_NUMBER_CUSTOM(..., ..., "...")
		// Parsing something of the form: LOCGEN_PERCENT/_GROUPED/_UNGROUPED(..., "...") or LOCGEN_PERCENT_CUSTOM(..., ..., "...")
		Buffer += TokenMarker.Len();

		const bool bIsCustom = TEXT_STRINGIFICATION_PEEK_MARKER(CustomSuffix);
		if (bIsCustom)
		{
			TEXT_STRINGIFICATION_SKIP_MARKER_LEN(CustomSuffix);
		}
		else if (TEXT_STRINGIFICATION_PEEK_MARKER(GroupedSuffix))
		{
			TEXT_STRINGIFICATION_SKIP_MARKER_LEN(GroupedSuffix);
			OutFormatOptions = FNumberFormattingOptions::DefaultWithGrouping();
		}
		else if (TEXT_STRINGIFICATION_PEEK_MARKER(UngroupedSuffix))
		{
			TEXT_STRINGIFICATION_SKIP_MARKER_LEN(UngroupedSuffix);
			OutFormatOptions = FNumberFormattingOptions::DefaultNoGrouping();
		}
		else
		{
			OutFormatOptions.Reset();
		}

		// Skip whitespace before the opening bracket, and then step over it
		TEXT_STRINGIFICATION_SKIP_WHITESPACE_AND_CHAR('(');

		// Skip whitespace before the value, and then read out the number
		TEXT_STRINGIFICATION_SKIP_WHITESPACE();
		TEXT_STRINGIFICATION_READ_NUMBER(OutSourceValue);

		if (bIsCustom)
		{
			// Skip whitespace before the comma, and then step over it
			TEXT_STRINGIFICATION_SKIP_WHITESPACE_AND_CHAR(',');

			// Skip any whitespace before the value, and then read the custom format options
			FNumberFormattingOptions LocalFormatOptions;
			TEXT_STRINGIFICATION_SKIP_WHITESPACE();
			TEXT_STRINGIFICATION_FUNC_MODIFY_BUFFER_AND_VALIDATE(ReadNumberFormattingOptionsFromBuffer, LocalFormatOptions);
			OutFormatOptions = MoveTemp(LocalFormatOptions);
		}

		// Skip whitespace before the comma, and then step over it
		TEXT_STRINGIFICATION_SKIP_WHITESPACE_AND_CHAR(',');

		// Skip whitespace before the value, and then read out the quoted culture name
		FString CultureNameString;
		TEXT_STRINGIFICATION_SKIP_WHITESPACE();
		TEXT_STRINGIFICATION_READ_QUOTED_STRING(CultureNameString);
		OutTargetCulture = (CultureNameString.IsEmpty()) ? nullptr : FInternationalization::Get().GetCulture(CultureNameString);

		// Skip whitespace before the closing bracket, and then step over it
		TEXT_STRINGIFICATION_SKIP_WHITESPACE_AND_CHAR(')');

		return Buffer;
	}

	return nullptr;
}

void WriteDateTimeToBuffer(FString& Buffer, const TCHAR* TokenMarker, const FDateTime& DateTime, const EDateTimeStyle::Type* DateStylePtr, const EDateTimeStyle::Type* TimeStylePtr, const FString* CustomPattern, const FString& TimeZone, FCulturePtr TargetCulture, const bool bStripPackageNamespace)
{
	auto WriteDateTimeStyle = [](FString& OutValueBuffer, const EDateTimeStyle::Type& InValue)
	{
		WriteScopedEnumToBuffer(OutValueBuffer, TEXT("EDateTimeStyle::"), InValue);
	};

	const bool bIsCustom = DateStylePtr && *DateStylePtr == EDateTimeStyle::Custom;
	const bool bIsInvariantTz = TimeZone == FText::GetInvariantTimeZone();

	FString Suffix;
	if (bIsCustom)
	{
		Suffix += CustomSuffix;
	}
	if (bIsInvariantTz)
	{
		Suffix += LocalSuffix;
	}
	else
	{
		Suffix += UtcSuffix;
	}

	// Produces LOCGEN_DATE_UTC(..., ..., "...", "...") or LOCGEN_DATE_LOCAL(..., ..., "...")
	// Produces LOCGEN_TIME_UTC(..., ..., "...", "...") or LOCGEN_TIME_LOCAL(..., ..., "...")
	// Produces LOCGEN_DATETIME_UTC(..., ..., ..., "...", "...") or LOCGEN_DATETIME_LOCAL(..., ..., ..., "...")
	// Produces LOCGEN_DATETIME_CUSTOM_UTC(..., "...", "...", "...") or LOCGEN_DATETIME_CUSTOM_LOCAL(..., "...", "...")
	Buffer += TokenMarker;
	Buffer += Suffix;
	Buffer += TEXT("(");
	FFormatArgumentValue(DateTime.ToUnixTimestamp()).ToExportedString(Buffer, bStripPackageNamespace);
	if (bIsCustom)
	{
		Buffer += TEXT(", \"");
		Buffer += CustomPattern ? CustomPattern->ReplaceCharWithEscapedChar() : FString();
		Buffer += TEXT("\"");
	}
	else
	{
		if (DateStylePtr)
		{
			Buffer += TEXT(", ");
			WriteDateTimeStyle(Buffer, *DateStylePtr);
		}
		if (TimeStylePtr)
		{
			Buffer += TEXT(", ");
			WriteDateTimeStyle(Buffer, *TimeStylePtr);
		}
	}
	if (!bIsInvariantTz)
	{
		Buffer += TEXT(", \"");
		Buffer += TimeZone.ReplaceCharWithEscapedChar();
		Buffer += TEXT("\"");
	}
	Buffer += TEXT(", \"");
	if (TargetCulture)
	{
		Buffer += TargetCulture->GetName().ReplaceCharWithEscapedChar();
	}
	Buffer += TEXT("\")");
}

const TCHAR* ReadDateTimeFromBuffer(const TCHAR* Buffer, const FString& TokenMarker, FDateTime& OutDateTime, EDateTimeStyle::Type* OutDateStylePtr, EDateTimeStyle::Type* OutTimeStylePtr, FString* OutCustomPattern, FString& OutTimeZone, FCulturePtr& OutTargetCulture)
{
	auto ReadDateTimeStyle = [](const TCHAR* InValueBuffer, EDateTimeStyle::Type& OutValue) -> const TCHAR*
	{
		static const FString DateTimeStyleMarker = "EDateTimeStyle::";
		return ReadScopedEnumFromBuffer(InValueBuffer, DateTimeStyleMarker, OutValue);
	};

	if (PeekMarker(Buffer, *TokenMarker, TokenMarker.Len()))
	{
		// Parsing something of the form: LOCGEN_DATE_UTC(..., ..., "...", "...") or LOCGEN_DATE_LOCAL(..., ..., "...")
		// Parsing something of the form: LOCGEN_TIME_UTC(..., ..., "...", "...") or LOCGEN_TIME_LOCAL(..., ..., "...")
		// Parsing something of the form: LOCGEN_DATETIME_UTC(..., ..., ..., "...", "...") or LOCGEN_DATETIME_LOCAL(..., ..., ..., "...")
		// Parsing something of the form: LOCGEN_DATETIME_CUSTOM_UTC(..., "...", "...", "...") or LOCGEN_DATETIME_CUSTOM_LOCAL(..., "...", "...")
		Buffer += TokenMarker.Len();

		const bool bIsCustom = TEXT_STRINGIFICATION_PEEK_MARKER(CustomSuffix);
		if (bIsCustom)
		{
			TEXT_STRINGIFICATION_SKIP_MARKER_LEN(CustomSuffix);
		}

		if (TEXT_STRINGIFICATION_PEEK_MARKER(LocalSuffix))
		{
			TEXT_STRINGIFICATION_SKIP_MARKER_LEN(LocalSuffix);
			OutTimeZone = FText::GetInvariantTimeZone();
		}
		else if (TEXT_STRINGIFICATION_PEEK_MARKER(UtcSuffix))
		{
			TEXT_STRINGIFICATION_SKIP_MARKER_LEN(UtcSuffix);
			OutTimeZone.Reset();
		}
		else
		{
			return nullptr;
		}

		// Skip whitespace before the opening bracket, and then step over it
		TEXT_STRINGIFICATION_SKIP_WHITESPACE_AND_CHAR('(');

		// Skip whitespace before the value, and then read out the number
		FFormatArgumentValue UnixTimestampValue;
		TEXT_STRINGIFICATION_SKIP_WHITESPACE();
		TEXT_STRINGIFICATION_READ_NUMBER(UnixTimestampValue);

		switch (UnixTimestampValue.GetType())
		{
		case EFormatArgumentType::Int:
			OutDateTime = FDateTime::FromUnixTimestamp(UnixTimestampValue.GetIntValue());
			break;
		case EFormatArgumentType::UInt:
			OutDateTime = FDateTime::FromUnixTimestamp(UnixTimestampValue.GetUIntValue());
			break;
		case EFormatArgumentType::Float:
			OutDateTime = FDateTime::FromUnixTimestamp((int64)UnixTimestampValue.GetFloatValue());
			break;
		case EFormatArgumentType::Double:
			OutDateTime = FDateTime::FromUnixTimestamp((int64)UnixTimestampValue.GetDoubleValue());
			break;
		default:
			return nullptr;
		}

		if (bIsCustom)
		{
			if (OutDateStylePtr)
			{
				*OutDateStylePtr = EDateTimeStyle::Custom;
			}
			if (OutTimeStylePtr)
			{
				*OutTimeStylePtr = EDateTimeStyle::Custom;
			}

			// Skip whitespace before the comma, and then step over it
			TEXT_STRINGIFICATION_SKIP_WHITESPACE_AND_CHAR(',');

			// Skip whitespace before the value, and then read out the quoted custom pattern
			FString CustomPatternTempString;
			FString& CustomPatternStringRef = OutCustomPattern ? *OutCustomPattern : CustomPatternTempString;
			TEXT_STRINGIFICATION_SKIP_WHITESPACE();
			TEXT_STRINGIFICATION_READ_QUOTED_STRING(CustomPatternStringRef);
		}
		else
		{
			if (OutDateStylePtr)
			{
				// Skip whitespace before the comma, then step over it
				TEXT_STRINGIFICATION_SKIP_WHITESPACE_AND_CHAR(',');

				// Skip any whitespace before the value, and then read the date style
				TEXT_STRINGIFICATION_SKIP_WHITESPACE();
				TEXT_STRINGIFICATION_FUNC_MODIFY_BUFFER_AND_VALIDATE(ReadDateTimeStyle, *OutDateStylePtr);
			}

			if (OutTimeStylePtr)
			{
				// Skip whitespace before the comma, then step over it
				TEXT_STRINGIFICATION_SKIP_WHITESPACE_AND_CHAR(',');

				// Skip any whitespace before the value, and then read the time style
				TEXT_STRINGIFICATION_SKIP_WHITESPACE();
				TEXT_STRINGIFICATION_FUNC_MODIFY_BUFFER_AND_VALIDATE(ReadDateTimeStyle, *OutTimeStylePtr);
			}
		}

		if (OutTimeZone.IsEmpty())
		{
			// Skip whitespace before the comma, and then step over it
			TEXT_STRINGIFICATION_SKIP_WHITESPACE_AND_CHAR(',');

			// Skip whitespace before the value, and then read out the quoted timezone name
			TEXT_STRINGIFICATION_SKIP_WHITESPACE();
			TEXT_STRINGIFICATION_READ_QUOTED_STRING(OutTimeZone);
		}

		// Skip whitespace before the comma, and then step over it
		TEXT_STRINGIFICATION_SKIP_WHITESPACE_AND_CHAR(',');

		// Skip whitespace before the value, and then read out the quoted culture name
		FString CultureNameString;
		TEXT_STRINGIFICATION_SKIP_WHITESPACE();
		TEXT_STRINGIFICATION_READ_QUOTED_STRING(CultureNameString);
		OutTargetCulture = (CultureNameString.IsEmpty()) ? nullptr : FInternationalization::Get().GetCulture(CultureNameString);

		// Skip whitespace before the closing bracket, and then step over it
		TEXT_STRINGIFICATION_SKIP_WHITESPACE_AND_CHAR(')');

		return Buffer;
	}

	return nullptr;
}

typedef TFunctionRef<void(const FString*, const FFormatArgumentValue&)> FTextFormatArgumentEnumeratorCallback;
void WriteTextFormatToBuffer(FString& Buffer, const FString& TokenMarker, const FTextFormat& SourceFmt, const bool bStripPackageNamespace, TFunctionRef<void(FTextFormatArgumentEnumeratorCallback)> ArgumentEnumerator)
{
	// Produces LOCGEN_FORMAT_NAMED(..., [...]) or LOCGEN_FORMAT_ORDERED(..., [...])
	Buffer += TokenMarker;
	Buffer += TEXT("(");
	FTextStringHelper::WriteToBuffer(Buffer, SourceFmt.GetSourceText(), /*bRequireQuotes*/true, bStripPackageNamespace);
	ArgumentEnumerator([&Buffer, bStripPackageNamespace](const FString* InKey, const FFormatArgumentValue& InValue)
	{
		if (InKey)
		{
			Buffer += TEXT(", \"");
			Buffer += *InKey;
			Buffer += TEXT("\"");
		}

		Buffer += TEXT(", ");
		InValue.ToExportedString(Buffer, bStripPackageNamespace);
	});
	Buffer += TEXT(")");
}

}	// namespace TextStringificationUtil

///////////////////////////////////////
// FTextHistory

#if WITH_EDITORONLY_DATA
const FString& FTextHistory::GetDevNotes() const
{
	static const FString EmptyNotes = TEXT("");
	return EmptyNotes;
}
#endif

uint16 FTextHistory::GetGlobalHistoryRevision() const
{
	UE::TScopeLock _(Mutex);
	return GlobalRevision;
}

uint16 FTextHistory::GetLocalHistoryRevision() const
{
	UE::TScopeLock _(Mutex);
	return LocalRevision;
}

void FTextHistory::UpdateDisplayStringIfOutOfDate()
{
	if (CanUpdateDisplayString())
	{
		uint16 CurrentGlobalRevision = 0;
		uint16 CurrentLocalRevision = 0;
		FTextLocalizationManager::Get().GetTextRevisions(GetTextId(), CurrentGlobalRevision, CurrentLocalRevision);

		// GlobalRevision and LocalRevision can be updated by concurrent threads!
		UE::TScopeLock _(Mutex);

		if (GlobalRevision != CurrentGlobalRevision || LocalRevision != CurrentLocalRevision)
		{
			GlobalRevision = CurrentGlobalRevision;
			LocalRevision = CurrentLocalRevision;

			UpdateDisplayString();
		}
	}
}

void FTextHistory::MarkDisplayStringOutOfDate()
{
	// GlobalRevision and LocalRevision can be updated by concurrent threads!
	UE::TScopeLock _(Mutex);

	GlobalRevision = 0;
	LocalRevision = 0;
}

void FTextHistory::MarkDisplayStringUpToDate()
{
	const bool bCanUpdate = CanUpdateDisplayString();

	// GlobalRevision and LocalRevision can be updated by concurrent threads!
	UE::TScopeLock _(Mutex);

	if (bCanUpdate)
	{
		FTextLocalizationManager::Get().GetTextRevisions(GetTextId(), GlobalRevision, LocalRevision);
	}
	else
	{
		GlobalRevision = 0;
		LocalRevision = 0;
	}
}

///////////////////////////////////////
// FTextHistory_Base

FTextHistory_Base::FTextHistory_Base(const FTextId& InTextId, FString&& InSourceString)
	: TextId(InTextId)
	, SourceString(MoveTemp(InSourceString))
{
}

FTextHistory_Base::FTextHistory_Base(const FTextId& InTextId, FString&& InSourceString, FTextConstDisplayStringPtr&& InLocalizedString)
	: TextId(InTextId)
	, SourceString(MoveTemp(InSourceString))
	, LocalizedString(MoveTemp(InLocalizedString))
{
	MarkDisplayStringUpToDate();
}

#if WITH_EDITORONLY_DATA
FTextHistory_Base::FTextHistory_Base(const FTextId& InTextId, FString&& InSourceString, FString&& InDevNotes)
	: TextId(InTextId)
	, SourceString(MoveTemp(InSourceString))
	, DevNotes(MoveTemp(InDevNotes))
{
}
#endif

FTextId FTextHistory_Base::GetTextId() const
{
	return TextId;
}

FTextConstDisplayStringPtr FTextHistory_Base::GetLocalizedString() const
{
	return LocalizedString;
}

const FString& FTextHistory_Base::GetSourceString() const
{
	return SourceString;
}

const FString& FTextHistory_Base::GetDisplayString() const
{
	return LocalizedString ? *LocalizedString : SourceString;
}

FString FTextHistory_Base::BuildInvariantDisplayString() const
{
	return SourceString;
}

bool FTextHistory_Base::IdenticalTo(const FTextHistory& Other, const ETextIdenticalModeFlags CompareModeFlags) const
{
	const FTextHistory_Base& CastOther = static_cast<const FTextHistory_Base&>(Other);
	return false; // No further comparison needed as FText::IdenticalTo already handles this case
}

void FTextHistory_Base::Serialize(FStructuredArchive::FRecord Record)
{
	FArchive& BaseArchive = Record.GetUnderlyingArchive();

	if(BaseArchive.IsLoading())
	{
		FTextKey Namespace;
		Namespace.SerializeAsString(Record.EnterField(TEXT("Namespace")));

		FTextKey Key;
		Key.SerializeAsString(Record.EnterField(TEXT("Key")));

		Record << SA_VALUE(TEXT("SourceString"), SourceString);

#if USE_STABLE_LOCALIZATION_KEYS
		// Make sure the package namespace for this text property is up-to-date
		// We do this on load (as well as save) to handle cases where data is being duplicated, as it will be written by one package and loaded into another
		if (GIsEditor && !Record.GetUnderlyingArchive().HasAnyPortFlags(PPF_DuplicateVerbatim | PPF_DuplicateForPIE))
		{
			const FString PackageNamespace = TextNamespaceUtil::GetPackageNamespace(BaseArchive);
			if (!PackageNamespace.IsEmpty())
			{
				const FString NamespaceStr = Namespace.ToString();
				const FString FullNamespace = TextNamespaceUtil::BuildFullNamespace(NamespaceStr, PackageNamespace);
				if (!NamespaceStr.Equals(FullNamespace, ESearchCase::CaseSensitive))
				{
					// We may assign a new key when loading if we don't have the correct package namespace in order to avoid identity conflicts when instancing (which duplicates without any special flags)
					// This can happen if an asset was duplicated (and keeps the same keys) but later both assets are instanced into the same world (causing them to both take the worlds package id, and conflict with each other)
					Namespace = FullNamespace;
					Key = FGuid::NewGuid().ToString();
				}
			}
		}
#endif // USE_STABLE_LOCALIZATION_KEYS
#if WITH_EDITOR
		if (!GIsEditor)
		{
			// Strip the package localization ID to match how text works at runtime (properties do this when saving during cook)
			Namespace = TextNamespaceUtil::StripPackageNamespace(Namespace.ToString());
		}
#endif // WITH_EDITOR

		TextId = FTextId(Namespace, Key);
		LocalizedString.Reset();
		MarkDisplayStringOutOfDate();
	}
	else if(BaseArchive.IsSaving())
	{
		FTextKey Namespace = TextId.GetNamespace();
		FTextKey Key = TextId.GetKey();

		if (BaseArchive.IsCooking())
		{
			// We strip the package localization off the serialized text for a cooked game, as they're not used at runtime
			Namespace = TextNamespaceUtil::StripPackageNamespace(Namespace.ToString());
		}
		else
		{
#if USE_STABLE_LOCALIZATION_KEYS
			// Make sure the package namespace for this text property is up-to-date
			if (GIsEditor && !BaseArchive.HasAnyPortFlags(PPF_DuplicateVerbatim | PPF_DuplicateForPIE))
			{
				const FString PackageNamespace = TextNamespaceUtil::GetPackageNamespace(BaseArchive);
				if (!PackageNamespace.IsEmpty())
				{
					const FString NamespaceStr = Namespace.ToString();
					const FString FullNamespace = TextNamespaceUtil::BuildFullNamespace(NamespaceStr, PackageNamespace);
					if (!NamespaceStr.Equals(FullNamespace, ESearchCase::CaseSensitive))
					{
						// We may assign a new key when saving if we don't have the correct package namespace in order to avoid identity conflicts when instancing (which duplicates without any special flags)
						// This can happen if an asset was duplicated (and keeps the same keys) but later both assets are instanced into the same world (causing them to both take the worlds package id, and conflict with each other)
						Namespace = FullNamespace;
						Key = FGuid::NewGuid().ToString();
					}
				}
			}
#endif // USE_STABLE_LOCALIZATION_KEYS

#if WITH_EDITOR
			// If this has no key, give it a GUID for a key
			if (GIsEditor && Key.IsEmpty() && (BaseArchive.IsPersistent() && !BaseArchive.HasAnyPortFlags(PPF_Duplicate)))
			{
				Key = FGuid::NewGuid().ToString();
			}

			// If the ID changed, and this is a persistent archive with a linker (meaning we're saving a package to disk), then apply the updated ID to the in-memory state
			if (GIsEditor && BaseArchive.IsPersistent() && BaseArchive.GetLinker() && (TextId.GetNamespace() != Namespace || TextId.GetKey() != Key))
			{
				TextId = FTextId(Namespace, Key);
				LocalizedString.Reset();
				MarkDisplayStringOutOfDate();
			}
#endif // WITH_EDITOR
		}

		// Serialize the Namespace
		Namespace.SerializeAsString(Record.EnterField(TEXT("Namespace")));

		// Serialize the Key
		Key.SerializeAsString(Record.EnterField(TEXT("Key")));

		// Serialize the SourceString
		Record << SA_VALUE(TEXT("SourceString"), SourceString);
	}

#if WITH_EDITORONLY_DATA || IS_PROGRAM
	BaseArchive.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	if (BaseArchive.CustomVer(FFortniteMainBranchObjectVersion::GUID) >= FFortniteMainBranchObjectVersion::AddDevNotesToFText && !BaseArchive.IsFilterEditorOnly() && !BaseArchive.IsNetArchive())
	{
		if (BaseArchive.IsLoading())
		{
			FString LoadedDevNotes;
			Record << SA_VALUE(TEXT("DevNotes"), LoadedDevNotes);
#if WITH_EDITORONLY_DATA
			DevNotes = MoveTemp(LoadedDevNotes);
#endif
		}
		else if (BaseArchive.IsSaving())
		{
#if WITH_EDITORONLY_DATA
			Record << SA_VALUE(TEXT("DevNotes"), DevNotes);
#else
			FString EmptyDevNotes;
			Record << SA_VALUE(TEXT("DevNotes"), EmptyDevNotes);
#endif
		}
	}	
#endif
}

bool FTextHistory_Base::CanUpdateDisplayString()
{
	return FTextLocalizationManager::IsDisplayStringSupportEnabled() && !TextId.IsEmpty();
}

void FTextHistory_Base::UpdateDisplayString()
{
	check(!TextId.IsEmpty()); // CanUpdateDisplayString should prevent UpdateDisplayString being called

	// Create a temp to hold the old value in case we abort, in which case we assign out of the OPEN so the old value will be preserved
	FTextConstDisplayStringPtr NewLocalizedString;
	UE_AUTORTFM_OPEN
	{
		NewLocalizedString = FTextLocalizationManager::Get().GetDisplayString(TextId.GetNamespace(), TextId.GetKey(), SourceString.IsEmpty() ? nullptr : &SourceString);
	};

	LocalizedString = NewLocalizedString;
}

#if WITH_EDITORONLY_DATA
const FString& FTextHistory_Base::GetDevNotes() const
{
	return DevNotes;
}
#endif

bool FTextHistory_Base::StaticShouldReadFromBuffer(const TCHAR* Buffer)
{
	return TEXT_STRINGIFICATION_PEEK_MARKER(TextStringificationUtil::NsLocTextMarker)
		|| TEXT_STRINGIFICATION_PEEK_MARKER(TextStringificationUtil::LocTextMarker)
		|| TEXT_STRINGIFICATION_PEEK_MARKER(TextStringificationUtil::NsNoteLocTextMarker)
		|| TEXT_STRINGIFICATION_PEEK_MARKER(TextStringificationUtil::NoteLocTextMarker);
}

const TCHAR* FTextHistory_Base::ReadFromBuffer(const TCHAR* Buffer, const TCHAR* TextNamespace, const TCHAR* PackageNamespace)
{
#define LOC_DEFINE_REGION
	const bool bIsNsNoteLocTextMarker = TEXT_STRINGIFICATION_PEEK_MARKER(TextStringificationUtil::NsNoteLocTextMarker);
	if (TEXT_STRINGIFICATION_PEEK_MARKER(TextStringificationUtil::NsLocTextMarker) || bIsNsNoteLocTextMarker)
	{
		// Parsing something of the form: NSLOCTEXT("...", "...", "...") or NSNOTELOCTEXT("...", "...", "...", "...")
		if (bIsNsNoteLocTextMarker)
		{
			TEXT_STRINGIFICATION_SKIP_MARKER_LEN(TextStringificationUtil::NsNoteLocTextMarker);
		}
		else
		{
			TEXT_STRINGIFICATION_SKIP_MARKER_LEN(TextStringificationUtil::NsLocTextMarker);
		}

		// Skip whitespace before the opening bracket, and then step over it
		TEXT_STRINGIFICATION_SKIP_WHITESPACE_AND_CHAR('(');

		// Skip whitespace before the value, and then read out the quoted namespace
		FString NamespaceString;
		TEXT_STRINGIFICATION_SKIP_WHITESPACE();
		TEXT_STRINGIFICATION_READ_QUOTED_STRING(NamespaceString);

		// Skip whitespace before the comma, and then step over it
		TEXT_STRINGIFICATION_SKIP_WHITESPACE_AND_CHAR(',');

		// Skip whitespace before the value, and then read out the quoted key
		FString KeyString;
		TEXT_STRINGIFICATION_SKIP_WHITESPACE();
		TEXT_STRINGIFICATION_READ_QUOTED_STRING(KeyString);

		// Skip whitespace before the comma, and then step over it
		TEXT_STRINGIFICATION_SKIP_WHITESPACE_AND_CHAR(',');

		// Skip whitespace before the value, and then read out the quoted source string
		TEXT_STRINGIFICATION_SKIP_WHITESPACE();
		TEXT_STRINGIFICATION_READ_QUOTED_STRING(SourceString);

		// Parse the DevNote if it exists
		if (bIsNsNoteLocTextMarker)
		{
			TEXT_STRINGIFICATION_SKIP_WHITESPACE_AND_CHAR(',');
			
			FString ReadDevNotes;
			TEXT_STRINGIFICATION_SKIP_WHITESPACE();
			TEXT_STRINGIFICATION_READ_QUOTED_STRING(ReadDevNotes);
#if WITH_EDITORONLY_DATA
			DevNotes = MoveTemp(ReadDevNotes);
#endif
		}

		// Skip whitespace before the closing bracket, and then step over it
		TEXT_STRINGIFICATION_SKIP_WHITESPACE_AND_CHAR(')');

		if (KeyString.IsEmpty())
		{
			KeyString = FGuid::NewGuid().ToString();
		}

#if USE_STABLE_LOCALIZATION_KEYS
		if (GIsEditor && PackageNamespace && *PackageNamespace)
		{
			const FString FullNamespace = TextNamespaceUtil::BuildFullNamespace(NamespaceString, PackageNamespace);
			if (!NamespaceString.Equals(FullNamespace, ESearchCase::CaseSensitive))
			{
				// We may assign a new key when importing if we don't have the correct package namespace in order to avoid identity conflicts when instancing (which duplicates without any special flags)
				// This can happen if an asset was duplicated (and keeps the same keys) but later both assets are instanced into the same world (causing them to both take the worlds package id, and conflict with each other)
				NamespaceString = FullNamespace;
				KeyString = FGuid::NewGuid().ToString();
			}
		}
#endif // USE_STABLE_LOCALIZATION_KEYS
		if (!GIsEditor)
		{
			// Strip the package localization ID to match how text works at runtime (properties do this when saving during cook)
			TextNamespaceUtil::StripPackageNamespaceInline(NamespaceString);
		}
		TextId = FTextId(NamespaceString, KeyString);
		LocalizedString.Reset();
		MarkDisplayStringOutOfDate();

		return Buffer;
	}
	
	const bool bIsNoteLocTextMarker = TEXT_STRINGIFICATION_PEEK_MARKER(TextStringificationUtil::NoteLocTextMarker);
	if (TEXT_STRINGIFICATION_PEEK_MARKER(TextStringificationUtil::LocTextMarker) || bIsNoteLocTextMarker)
	{
		// Parsing something of the form: LOCTEXT("...", "...") or NOTELOCTEXT("...", "...", "...")
		// This only exists as people sometimes do this in config files. We assume an empty namespace should be used
		if (bIsNoteLocTextMarker)
		{
			TEXT_STRINGIFICATION_SKIP_MARKER_LEN(TextStringificationUtil::NoteLocTextMarker);
		}
		else
		{
			TEXT_STRINGIFICATION_SKIP_MARKER_LEN(TextStringificationUtil::LocTextMarker);
		}

		// Skip whitespace before the opening bracket, and then step over it
		TEXT_STRINGIFICATION_SKIP_WHITESPACE_AND_CHAR('(');

		// Skip whitespace before the value, and then read out the quoted key
		FString KeyString;
		TEXT_STRINGIFICATION_SKIP_WHITESPACE();
		TEXT_STRINGIFICATION_READ_QUOTED_STRING(KeyString);

		// Skip whitespace before the comma, and then step over it
		TEXT_STRINGIFICATION_SKIP_WHITESPACE_AND_CHAR(',');

		// Skip whitespace before the value, and then read out the quoted source string
		TEXT_STRINGIFICATION_SKIP_WHITESPACE();
		TEXT_STRINGIFICATION_READ_QUOTED_STRING(SourceString);

		// Parse the DevNote if it exists
		if (bIsNoteLocTextMarker)
		{
			TEXT_STRINGIFICATION_SKIP_WHITESPACE_AND_CHAR(',');
			
			FString ReadDevNotes;
			TEXT_STRINGIFICATION_SKIP_WHITESPACE();
			TEXT_STRINGIFICATION_READ_QUOTED_STRING(ReadDevNotes);
#if WITH_EDITORONLY_DATA
			DevNotes = MoveTemp(ReadDevNotes);
#endif
		}

		// Skip whitespace before the closing bracket, and then step over it
		TEXT_STRINGIFICATION_SKIP_WHITESPACE_AND_CHAR(')');

		if (KeyString.IsEmpty())
		{
			KeyString = FGuid::NewGuid().ToString();
		}

		FString NamespaceString = (TextNamespace) ? TextNamespace : FString();
#if USE_STABLE_LOCALIZATION_KEYS
		if (GIsEditor && PackageNamespace && *PackageNamespace)
		{
			const FString FullNamespace = TextNamespaceUtil::BuildFullNamespace(NamespaceString, PackageNamespace);
			if (!NamespaceString.Equals(FullNamespace, ESearchCase::CaseSensitive))
			{
				// We may assign a new key when importing if we don't have the correct package namespace in order to avoid identity conflicts when instancing (which duplicates without any special flags)
				// This can happen if an asset was duplicated (and keeps the same keys) but later both assets are instanced into the same world (causing them to both take the worlds package id, and conflict with each other)
				NamespaceString = FullNamespace;
				KeyString = FGuid::NewGuid().ToString();
			}
		}
#endif // USE_STABLE_LOCALIZATION_KEYS
		if (!GIsEditor)
		{
			// Strip the package localization ID to match how text works at runtime (properties do this when saving during cook)
			TextNamespaceUtil::StripPackageNamespaceInline(NamespaceString);
		}
		TextId = FTextId(NamespaceString, KeyString);
		LocalizedString.Reset();
		MarkDisplayStringOutOfDate();

		return Buffer;
	}
#undef LOC_DEFINE_REGION

	return nullptr;
}

bool FTextHistory_Base::WriteToBuffer(FString& Buffer, const bool bStripPackageNamespace) const
{
	if (!TextId.IsEmpty())
	{
		FString Namespace = TextId.GetNamespace().ToString();
		FString Key = TextId.GetKey().ToString();
		if (bStripPackageNamespace)
		{
			TextNamespaceUtil::StripPackageNamespaceInline(Namespace);
		}

#define LOC_DEFINE_REGION
#if WITH_EDITORONLY_DATA
		const FString DevNotesToWrite = GetDevNotes().TrimStartAndEnd();
		if (!DevNotesToWrite.IsEmpty())
		{
			// Produces NSNOTELOCTEXT("...", "...", "...", "...")
			Buffer += TEXT("NSNOTELOCTEXT(\"");
			Buffer += Namespace.ReplaceCharWithEscapedChar();
			Buffer += TEXT("\", \"");
			Buffer += Key.ReplaceCharWithEscapedChar();
			Buffer += TEXT("\", \"");
			Buffer += SourceString.ReplaceCharWithEscapedChar();
			Buffer += TEXT("\", \"");
			Buffer += DevNotesToWrite.ReplaceCharWithEscapedChar();
			Buffer += TEXT("\")");
			
			return true;
		}
#endif
		// Produces NSLOCTEXT("...", "...", "...")
		Buffer += TEXT("NSLOCTEXT(\"");
		Buffer += Namespace.ReplaceCharWithEscapedChar();
		Buffer += TEXT("\", \"");
		Buffer += Key.ReplaceCharWithEscapedChar();
		Buffer += TEXT("\", \"");
		Buffer += SourceString.ReplaceCharWithEscapedChar();
		Buffer += TEXT("\")");
#undef LOC_DEFINE_REGION

		return true;
	}

	return false;
}

///////////////////////////////////////
// FTextHistory_Generated

FTextHistory_Generated::FTextHistory_Generated(FString&& InDisplayString)
	: DisplayString(MoveTemp(InDisplayString))
{
	MarkDisplayStringUpToDate();
}

const FString& FTextHistory_Generated::GetDisplayString() const
{
	return DisplayString;
}

void FTextHistory_Generated::Serialize(FStructuredArchive::FRecord Record)
{
	FArchive& BaseArchive = Record.GetUnderlyingArchive();

	if (BaseArchive.IsLoading())
	{
		MarkDisplayStringOutOfDate();
	}
}

void FTextHistory_Generated::UpdateDisplayString()
{
	DisplayString = BuildLocalizedDisplayString();
}

///////////////////////////////////////
// FTextHistory_NamedFormat

FTextHistory_NamedFormat::FTextHistory_NamedFormat(FString&& InDisplayString, FTextFormat&& InSourceFmt, FFormatNamedArguments&& InArguments)
	: FTextHistory_Generated(MoveTemp(InDisplayString))
	, SourceFmt(MoveTemp(InSourceFmt))
	, Arguments(MoveTemp(InArguments))
{
}

bool FTextHistory_NamedFormat::IdenticalTo(const FTextHistory& Other, const ETextIdenticalModeFlags CompareModeFlags) const
{
	const FTextHistory_NamedFormat& CastOther = static_cast<const FTextHistory_NamedFormat&>(Other);

	if (!SourceFmt.IdenticalTo(CastOther.SourceFmt, CompareModeFlags))
	{
		return false;
	}

	if (Arguments.Num() == CastOther.Arguments.Num())
	{
		bool bMatchesAllArgs = true;
		for (const auto& ArgNameDataPair : Arguments)
		{
			const FFormatArgumentValue* OtherArgData = CastOther.Arguments.Find(ArgNameDataPair.Key);
			bMatchesAllArgs &= (OtherArgData && ArgNameDataPair.Value.IdenticalTo(*OtherArgData, CompareModeFlags));
			if (!bMatchesAllArgs)
			{
				break;
			}
		}
		return bMatchesAllArgs;
	}

	return false;
}

FString FTextHistory_NamedFormat::BuildLocalizedDisplayString() const
{
	return FTextFormatter::FormatStr(SourceFmt, Arguments, true, false);
}

FString FTextHistory_NamedFormat::BuildInvariantDisplayString() const
{
	return FTextFormatter::FormatStr(SourceFmt, Arguments, true, true);
}

void FTextHistory_NamedFormat::Serialize(FStructuredArchive::FRecord Record)
{
	FTextHistory_Generated::Serialize(Record);

	FArchive& BaseArchive = Record.GetUnderlyingArchive();

	if (BaseArchive.IsSaving())
	{
		FText FormatText = SourceFmt.GetSourceText();
		Record.EnterField(TEXT("FormatText")) << FormatText;
	}
	else if (BaseArchive.IsLoading())
	{
		FText FormatText;
		Record.EnterField(TEXT("FormatText")) << FormatText;
		SourceFmt = FTextFormat(FormatText);
	}

	Record << SA_VALUE(TEXT("Arguments"), Arguments);
}

bool FTextHistory_NamedFormat::StaticShouldReadFromBuffer(const TCHAR* Buffer)
{
	return TEXT_STRINGIFICATION_PEEK_MARKER(TextStringificationUtil::LocGenFormatNamedMarker);
}

const TCHAR* FTextHistory_NamedFormat::ReadFromBuffer(const TCHAR* Buffer, const TCHAR* TextNamespace, const TCHAR* PackageNamespace)
{
	if (TEXT_STRINGIFICATION_PEEK_MARKER(TextStringificationUtil::LocGenFormatNamedMarker))
	{
		// Parsing something of the form: LOCGEN_FORMAT_NAMED(..., [...])
		TEXT_STRINGIFICATION_SKIP_MARKER_LEN(TextStringificationUtil::LocGenFormatNamedMarker);

		// Skip whitespace before the opening bracket, and then step over it
		TEXT_STRINGIFICATION_SKIP_WHITESPACE_AND_CHAR('(');

		// Skip whitespace before the value, and then read out the text
		FText FormatText;
		TEXT_STRINGIFICATION_SKIP_WHITESPACE();
		TEXT_STRINGIFICATION_FUNC_MODIFY_BUFFER_AND_VALIDATE(FTextStringHelper::ReadFromBuffer, FormatText, nullptr, nullptr, true);
		SourceFmt = FTextFormat(FormatText);

		// Read out arguments until we run out
		Arguments.Reset();
		for (;;)
		{
			// Skip whitespace and see if we've found a comma (for another argument)
			TEXT_STRINGIFICATION_SKIP_WHITESPACE();
			if (*Buffer != TEXT(','))
			{
				// Finished parsing
				break;
			}

			// Step over the comma
			++Buffer;

			// Skip whitespace before the value, and then read out the quoted argument name
			FString ArgumentName;
			TEXT_STRINGIFICATION_SKIP_WHITESPACE();
			TEXT_STRINGIFICATION_READ_QUOTED_STRING(ArgumentName);

			// Skip whitespace before the comma, then step over it
			TEXT_STRINGIFICATION_SKIP_WHITESPACE_AND_CHAR(',');

			// Skip whitespace before the value, and then read the new argument
			FFormatArgumentValue& ArgumentValue = Arguments.Add(MoveTemp(ArgumentName));
			TEXT_STRINGIFICATION_SKIP_WHITESPACE();
			TEXT_STRINGIFICATION_FUNC_MODIFY_BUFFER_AND_VALIDATE(ArgumentValue.FromExportedString);
		}

		// Skip whitespace before the closing bracket, and then step over it
		TEXT_STRINGIFICATION_SKIP_WHITESPACE_AND_CHAR(')');

		MarkDisplayStringOutOfDate();
		return Buffer;
	}

	return nullptr;
}

bool FTextHistory_NamedFormat::WriteToBuffer(FString& Buffer, const bool bStripPackageNamespace) const
{
	TextStringificationUtil::WriteTextFormatToBuffer(Buffer, TextStringificationUtil::LocGenFormatNamedMarker, SourceFmt, bStripPackageNamespace, [this](TextStringificationUtil::FTextFormatArgumentEnumeratorCallback Callback)
	{
		for (const auto& ArgumentPair : Arguments)
		{
			Callback(&ArgumentPair.Key, ArgumentPair.Value);
		}
	});
	return true;
}

void FTextHistory_NamedFormat::GetHistoricFormatData(const FText& InText, TArray<FHistoricTextFormatData>& OutHistoricFormatData) const
{
	// Process the formatting text in-case it's a recursive format
	FTextInspector::GetHistoricFormatData(SourceFmt.GetSourceText(), OutHistoricFormatData);

	for (auto It = Arguments.CreateConstIterator(); It; ++It)
	{
		const FFormatArgumentValue& ArgumentValue = It.Value();
		if (ArgumentValue.GetType() == EFormatArgumentType::Text)
		{
			// Process the text argument in-case it's a recursive format
			FTextInspector::GetHistoricFormatData(ArgumentValue.GetTextValue(), OutHistoricFormatData);
		}
	}

	// Add ourself now that we've processed any format dependencies
	OutHistoricFormatData.Emplace(InText, FTextFormat(SourceFmt), FFormatNamedArguments(Arguments));
}

///////////////////////////////////////
// FTextHistory_OrderedFormat

FTextHistory_OrderedFormat::FTextHistory_OrderedFormat(FString&& InDisplayString, FTextFormat&& InSourceFmt, FFormatOrderedArguments&& InArguments)
	: FTextHistory_Generated(MoveTemp(InDisplayString))
	, SourceFmt(MoveTemp(InSourceFmt))
	, Arguments(MoveTemp(InArguments))
{
}

bool FTextHistory_OrderedFormat::IdenticalTo(const FTextHistory& Other, const ETextIdenticalModeFlags CompareModeFlags) const
{
	const FTextHistory_OrderedFormat& CastOther = static_cast<const FTextHistory_OrderedFormat&>(Other);

	if (!SourceFmt.IdenticalTo(CastOther.SourceFmt, CompareModeFlags))
	{
		return false;
	}

	if (Arguments.Num() == CastOther.Arguments.Num())
	{
		bool bMatchesAllArgs = true;
		for (int32 ArgIndex = 0; ArgIndex < Arguments.Num() && bMatchesAllArgs; ++ArgIndex)
		{
			bMatchesAllArgs &= Arguments[ArgIndex].IdenticalTo(CastOther.Arguments[ArgIndex], CompareModeFlags);
		}
		return bMatchesAllArgs;
	}

	return false;
}

FString FTextHistory_OrderedFormat::BuildLocalizedDisplayString() const
{
	return FTextFormatter::FormatStr(SourceFmt, Arguments, true, false);
}

FString FTextHistory_OrderedFormat::BuildInvariantDisplayString() const
{
	return FTextFormatter::FormatStr(SourceFmt, Arguments, true, true);
}

void FTextHistory_OrderedFormat::Serialize(FStructuredArchive::FRecord Record)
{
	FTextHistory_Generated::Serialize(Record);

	FArchive& BaseArchive = Record.GetUnderlyingArchive();

	if (BaseArchive.IsSaving())
	{
		FText FormatText = SourceFmt.GetSourceText();
		Record.EnterField(TEXT("FormatText")) << FormatText;
	}
	else if (BaseArchive.IsLoading())
	{
		FText FormatText;
		Record.EnterField(TEXT("FormatText")) << FormatText;
		SourceFmt = FTextFormat(FormatText);
	}

	Record << SA_VALUE(TEXT("Arguments"), Arguments);
}

bool FTextHistory_OrderedFormat::StaticShouldReadFromBuffer(const TCHAR* Buffer)
{
	return TEXT_STRINGIFICATION_PEEK_MARKER(TextStringificationUtil::LocGenFormatOrderedMarker);
}

const TCHAR* FTextHistory_OrderedFormat::ReadFromBuffer(const TCHAR* Buffer, const TCHAR* TextNamespace, const TCHAR* PackageNamespace)
{
	if (TEXT_STRINGIFICATION_PEEK_MARKER(TextStringificationUtil::LocGenFormatOrderedMarker))
	{
		// Parsing something of the form: LOCGEN_FORMAT_ORDERED(..., [...])
		TEXT_STRINGIFICATION_SKIP_MARKER_LEN(TextStringificationUtil::LocGenFormatOrderedMarker);

		// Skip whitespace before the opening bracket, and then step over it
		TEXT_STRINGIFICATION_SKIP_WHITESPACE_AND_CHAR('(');

		// Skip whitespace before the value, and then read out the text
		FText FormatText;
		TEXT_STRINGIFICATION_SKIP_WHITESPACE();
		TEXT_STRINGIFICATION_FUNC_MODIFY_BUFFER_AND_VALIDATE(FTextStringHelper::ReadFromBuffer, FormatText, nullptr, nullptr, true);
		SourceFmt = FTextFormat(FormatText);

		// Read out arguments until we run out
		Arguments.Reset();
		for (;;)
		{
			// Skip whitespace and see if we've found a comma (for another argument)
			TEXT_STRINGIFICATION_SKIP_WHITESPACE();
			if (*Buffer != TEXT(','))
			{
				// Finished parsing
				break;
			}

			// Step over the comma
			++Buffer;

			// Skip whitespace before the value, and then read the new argument
			FFormatArgumentValue& ArgumentValue = Arguments.AddDefaulted_GetRef();
			TEXT_STRINGIFICATION_SKIP_WHITESPACE();
			TEXT_STRINGIFICATION_FUNC_MODIFY_BUFFER_AND_VALIDATE(ArgumentValue.FromExportedString);
		}

		// Skip whitespace before the closing bracket, and then step over it
		TEXT_STRINGIFICATION_SKIP_WHITESPACE_AND_CHAR(')');

		MarkDisplayStringOutOfDate();
		return Buffer;
	}

	return nullptr;
}

bool FTextHistory_OrderedFormat::WriteToBuffer(FString& Buffer, const bool bStripPackageNamespace) const
{
	TextStringificationUtil::WriteTextFormatToBuffer(Buffer, TextStringificationUtil::LocGenFormatOrderedMarker, SourceFmt, bStripPackageNamespace, [this](TextStringificationUtil::FTextFormatArgumentEnumeratorCallback Callback)
	{
		for (const FFormatArgumentValue& ArgumentValue : Arguments)
		{
			Callback(nullptr, ArgumentValue);
		}
	});
	return true;
}

void FTextHistory_OrderedFormat::GetHistoricFormatData(const FText& InText, TArray<FHistoricTextFormatData>& OutHistoricFormatData) const
{
	// Process the formatting text in-case it's a recursive format
	FTextInspector::GetHistoricFormatData(SourceFmt.GetSourceText(), OutHistoricFormatData);

	for (auto It = Arguments.CreateConstIterator(); It; ++It)
	{
		const FFormatArgumentValue& ArgumentValue = *It;
		if (ArgumentValue.GetType() == EFormatArgumentType::Text)
		{
			// Process the text argument in-case it's a recursive format
			FTextInspector::GetHistoricFormatData(ArgumentValue.GetTextValue(), OutHistoricFormatData);
		}
	}

	// Add ourself now that we've processed any format dependencies
	FFormatNamedArguments NamedArgs;
	NamedArgs.Reserve(Arguments.Num());
	for (int32 ArgIndex = 0; ArgIndex < Arguments.Num(); ++ArgIndex)
	{
		const FFormatArgumentValue& ArgumentValue = Arguments[ArgIndex];
		NamedArgs.Emplace(FString::FromInt(ArgIndex), ArgumentValue);
	}
	OutHistoricFormatData.Emplace(InText, FTextFormat(SourceFmt), MoveTemp(NamedArgs));
}

///////////////////////////////////////
// FTextHistory_ArgumentDataFormat

FTextHistory_ArgumentDataFormat::FTextHistory_ArgumentDataFormat(FString&& InDisplayString, FTextFormat&& InSourceFmt, TArray<FFormatArgumentData>&& InArguments)
	: FTextHistory_Generated(MoveTemp(InDisplayString))
	, SourceFmt(MoveTemp(InSourceFmt))
	, Arguments(MoveTemp(InArguments))
{
}

bool FTextHistory_ArgumentDataFormat::IdenticalTo(const FTextHistory& Other, const ETextIdenticalModeFlags CompareModeFlags) const
{
	const FTextHistory_ArgumentDataFormat& CastOther = static_cast<const FTextHistory_ArgumentDataFormat&>(Other);

	if (!SourceFmt.IdenticalTo(CastOther.SourceFmt, CompareModeFlags))
	{
		return false;
	}

	if (Arguments.Num() == CastOther.Arguments.Num())
	{
		bool bMatchesAllArgs = true;
		for (int32 ArgIndex = 0; ArgIndex < Arguments.Num() && bMatchesAllArgs; ++ArgIndex)
		{
			bMatchesAllArgs &= Arguments[ArgIndex].ToArgumentValue().IdenticalTo(CastOther.Arguments[ArgIndex].ToArgumentValue(), CompareModeFlags);
		}
		return bMatchesAllArgs;
	}

	return false;
}

FString FTextHistory_ArgumentDataFormat::BuildLocalizedDisplayString() const
{
	return FTextFormatter::FormatStr(SourceFmt, Arguments, true, false);
}

FString FTextHistory_ArgumentDataFormat::BuildInvariantDisplayString() const
{
	return FTextFormatter::FormatStr(SourceFmt, Arguments, true, true);
}

void FTextHistory_ArgumentDataFormat::Serialize(FStructuredArchive::FRecord Record)
{
	FTextHistory_Generated::Serialize(Record);

	FArchive& BaseArchive = Record.GetUnderlyingArchive();

	if (BaseArchive.IsSaving())
	{
		FText FormatText = SourceFmt.GetSourceText();
		Record.EnterField(TEXT("FormatText")) << FormatText;
	}
	else if (BaseArchive.IsLoading())
	{
		FText FormatText;
		Record.EnterField(TEXT("FormatText")) << FormatText;
		SourceFmt = FTextFormat(FormatText);
	}

	Record << SA_VALUE(TEXT("Arguments"), Arguments);
}

bool FTextHistory_ArgumentDataFormat::StaticShouldReadFromBuffer(const TCHAR* Buffer)
{
	return false;
}

const TCHAR* FTextHistory_ArgumentDataFormat::ReadFromBuffer(const TCHAR* Buffer, const TCHAR* TextNamespace, const TCHAR* PackageNamespace)
{
	return nullptr;
}

bool FTextHistory_ArgumentDataFormat::WriteToBuffer(FString& Buffer, const bool bStripPackageNamespace) const
{
	TextStringificationUtil::WriteTextFormatToBuffer(Buffer, TextStringificationUtil::LocGenFormatNamedMarker, SourceFmt, bStripPackageNamespace, [this](TextStringificationUtil::FTextFormatArgumentEnumeratorCallback Callback)
	{
		for (const FFormatArgumentData& Argument : Arguments)
		{
			Callback(&Argument.ArgumentName, Argument.ToArgumentValue());
		}
	});
	return true;
}

void FTextHistory_ArgumentDataFormat::GetHistoricFormatData(const FText& InText, TArray<FHistoricTextFormatData>& OutHistoricFormatData) const
{
	// Process the formatting text in-case it's a recursive format
	FTextInspector::GetHistoricFormatData(SourceFmt.GetSourceText(), OutHistoricFormatData);

	for (const FFormatArgumentData& ArgumentData : Arguments)
	{
		if (ArgumentData.ArgumentValueType == EFormatArgumentType::Text)
		{
			// Process the text argument in-case it's a recursive format
			FTextInspector::GetHistoricFormatData(ArgumentData.ArgumentValue, OutHistoricFormatData);
		}
	}

	// Add ourself now that we've processed any format dependencies
	FFormatNamedArguments NamedArgs;
	NamedArgs.Reserve(Arguments.Num());
	for (const FFormatArgumentData& ArgumentData : Arguments)
	{
		FFormatArgumentValue ArgumentValue;
		switch (ArgumentData.ArgumentValueType)
		{
		case EFormatArgumentType::Int:
			ArgumentValue = FFormatArgumentValue(ArgumentData.ArgumentValueInt);
			break;
		case EFormatArgumentType::Float:
			ArgumentValue = FFormatArgumentValue(ArgumentData.ArgumentValueFloat);
			break;
		case EFormatArgumentType::Double:
			ArgumentValue = FFormatArgumentValue(ArgumentData.ArgumentValueDouble);
			break;
		case EFormatArgumentType::Gender:
			ArgumentValue = FFormatArgumentValue(ArgumentData.ArgumentValueGender);
			break;
		default:
			ArgumentValue = FFormatArgumentValue(ArgumentData.ArgumentValue);
			break;
		}

		NamedArgs.Emplace(ArgumentData.ArgumentName, MoveTemp(ArgumentValue));
	}
	OutHistoricFormatData.Emplace(InText, FTextFormat(SourceFmt), MoveTemp(NamedArgs));
}

///////////////////////////////////////
// FTextHistory_FormatNumber

FTextHistory_FormatNumber::FTextHistory_FormatNumber(FString&& InDisplayString, FFormatArgumentValue InSourceValue, const FNumberFormattingOptions* const InFormatOptions, FCulturePtr InTargetCulture)
	: FTextHistory_Generated(MoveTemp(InDisplayString))
	, SourceValue(MoveTemp(InSourceValue))
	, FormatOptions()
	, TargetCulture(MoveTemp(InTargetCulture))
{
	if(InFormatOptions)
	{
		FormatOptions = *InFormatOptions;
	}
}

bool FTextHistory_FormatNumber::IdenticalTo(const FTextHistory& Other, const ETextIdenticalModeFlags CompareModeFlags) const
{
	const FTextHistory_FormatNumber& CastOther = static_cast<const FTextHistory_FormatNumber&>(Other);

	return SourceValue.IdenticalTo(CastOther.SourceValue, CompareModeFlags)
		&& FormatOptions.Get(FNumberFormattingOptions::DefaultWithGrouping()).IsIdentical(CastOther.FormatOptions.Get(FNumberFormattingOptions::DefaultWithGrouping()))
		&& TargetCulture == CastOther.TargetCulture;
}

void FTextHistory_FormatNumber::Serialize(FStructuredArchive::FRecord Record)
{
	FTextHistory_Generated::Serialize(Record);

	FArchive& BaseArchive = Record.GetUnderlyingArchive();

	Record << SA_VALUE(TEXT("SourceValue"), SourceValue);

	bool bHasFormatOptions = FormatOptions.IsSet();
	Record << SA_VALUE(TEXT("bHasFormatOptions"), bHasFormatOptions);

	if(BaseArchive.IsLoading())
	{
		if(bHasFormatOptions)
		{
			FormatOptions = FNumberFormattingOptions();
		}
		else
		{
			FormatOptions.Reset();
		}
	}
	if(bHasFormatOptions)
	{
		check(FormatOptions.IsSet());
		FNumberFormattingOptions& Options = FormatOptions.GetValue();
		Record << SA_VALUE(TEXT("Options"), Options);
	}

	if(BaseArchive.IsSaving())
	{
		FString CultureName = TargetCulture.IsValid()? TargetCulture->GetName() : FString();
		Record << SA_VALUE(TEXT("CultureName"), CultureName);
	}
	else if(BaseArchive.IsLoading())
	{
		FString CultureName;
		Record << SA_VALUE(TEXT("CultureName"), CultureName);

		if(!CultureName.IsEmpty())
		{
			TargetCulture = FInternationalization::Get().GetCulture(CultureName);
		}
	}
}

FString FTextHistory_FormatNumber::BuildNumericDisplayString(const FDecimalNumberFormattingRules& InFormattingRules, const int32 InValueMultiplier) const
{
	check(InValueMultiplier > 0);

	const FNumberFormattingOptions& FormattingOptions = (FormatOptions.IsSet()) ? FormatOptions.GetValue() : InFormattingRules.CultureDefaultFormattingOptions;
	switch (SourceValue.GetType())
	{
	case EFormatArgumentType::Int:
		return FastDecimalFormat::NumberToString(SourceValue.GetIntValue() * static_cast<int64>(InValueMultiplier), InFormattingRules, FormattingOptions);
	case EFormatArgumentType::UInt:
		return FastDecimalFormat::NumberToString(SourceValue.GetUIntValue() * static_cast<uint64>(InValueMultiplier), InFormattingRules, FormattingOptions);
	case EFormatArgumentType::Float:
		return FastDecimalFormat::NumberToString(SourceValue.GetFloatValue() * static_cast<float>(InValueMultiplier), InFormattingRules, FormattingOptions);
	case EFormatArgumentType::Double:
		return FastDecimalFormat::NumberToString(SourceValue.GetDoubleValue() * static_cast<double>(InValueMultiplier), InFormattingRules, FormattingOptions);
	default:
		break;
	}
	return FString();
}

///////////////////////////////////////
// FTextHistory_AsNumber

FTextHistory_AsNumber::FTextHistory_AsNumber(FString&& InDisplayString, FFormatArgumentValue InSourceValue, const FNumberFormattingOptions* const InFormatOptions, FCulturePtr InTargetCulture)
	: FTextHistory_FormatNumber(MoveTemp(InDisplayString), MoveTemp(InSourceValue), InFormatOptions, MoveTemp(InTargetCulture))
{
}

FString FTextHistory_AsNumber::BuildLocalizedDisplayString() const
{
	FInternationalization& I18N = FInternationalization::Get();
	checkf(I18N.IsInitialized() == true, TEXT("FInternationalization is not initialized. An FText formatting method was likely used in static object initialization - this is not supported."));
	const FCulture& Culture = TargetCulture.IsValid() ? *TargetCulture : *I18N.GetCurrentLocale();

	const FDecimalNumberFormattingRules& FormattingRules = Culture.GetDecimalNumberFormattingRules();
	return BuildNumericDisplayString(FormattingRules);
}

FString FTextHistory_AsNumber::BuildInvariantDisplayString() const
{
	FInternationalization& I18N = FInternationalization::Get();
	checkf(I18N.IsInitialized() == true, TEXT("FInternationalization is not initialized. An FText formatting method was likely used in static object initialization - this is not supported."));
	const FCulture& Culture = *I18N.GetInvariantCulture();

	const FDecimalNumberFormattingRules& FormattingRules = Culture.GetDecimalNumberFormattingRules();
	return BuildNumericDisplayString(FormattingRules);
}

void FTextHistory_AsNumber::Serialize(FStructuredArchive::FRecord Record)
{
	FTextHistory_FormatNumber::Serialize(Record);
}

bool FTextHistory_AsNumber::StaticShouldReadFromBuffer(const TCHAR* Buffer)
{
	return TEXT_STRINGIFICATION_PEEK_MARKER(TextStringificationUtil::LocGenNumberMarker);
}

const TCHAR* FTextHistory_AsNumber::ReadFromBuffer(const TCHAR* Buffer, const TCHAR* TextNamespace, const TCHAR* PackageNamespace)
{
	static const FString TokenMarker = TextStringificationUtil::LocGenNumberMarker;
	TEXT_STRINGIFICATION_FUNC_MODIFY_BUFFER_AND_VALIDATE(TextStringificationUtil::ReadNumberOrPercentFromBuffer, TokenMarker, SourceValue, FormatOptions, TargetCulture);
	MarkDisplayStringOutOfDate();
	return Buffer;
}

bool FTextHistory_AsNumber::WriteToBuffer(FString& Buffer, const bool bStripPackageNamespace) const
{
	TextStringificationUtil::WriteNumberOrPercentToBuffer(Buffer, TextStringificationUtil::LocGenNumberMarker, SourceValue, FormatOptions, TargetCulture, bStripPackageNamespace);
	return true;
}

bool FTextHistory_AsNumber::GetHistoricNumericData(const FText& InText, FHistoricTextNumericData& OutHistoricNumericData) const
{
	OutHistoricNumericData = FHistoricTextNumericData(FHistoricTextNumericData::EType::AsNumber, SourceValue, FormatOptions);
	return true;
}

///////////////////////////////////////
// FTextHistory_AsPercent

FTextHistory_AsPercent::FTextHistory_AsPercent(FString&& InDisplayString, FFormatArgumentValue InSourceValue, const FNumberFormattingOptions* const InFormatOptions, FCulturePtr InTargetCulture)
	: FTextHistory_FormatNumber(MoveTemp(InDisplayString), MoveTemp(InSourceValue), InFormatOptions, MoveTemp(InTargetCulture))
{
}

FString FTextHistory_AsPercent::BuildLocalizedDisplayString() const
{
	FInternationalization& I18N = FInternationalization::Get();
	checkf(I18N.IsInitialized() == true, TEXT("FInternationalization is not initialized. An FText formatting method was likely used in static object initialization - this is not supported."));
	const FCulture& Culture = TargetCulture.IsValid() ? *TargetCulture : *I18N.GetCurrentLocale();

	const FDecimalNumberFormattingRules& FormattingRules = Culture.GetPercentFormattingRules();
	return BuildNumericDisplayString(FormattingRules, 100);
}

FString FTextHistory_AsPercent::BuildInvariantDisplayString() const
{
	FInternationalization& I18N = FInternationalization::Get();
	checkf(I18N.IsInitialized() == true, TEXT("FInternationalization is not initialized. An FText formatting method was likely used in static object initialization - this is not supported."));
	const FCulture& Culture = *I18N.GetInvariantCulture();

	const FDecimalNumberFormattingRules& FormattingRules = Culture.GetPercentFormattingRules();
	return BuildNumericDisplayString(FormattingRules, 100);
}

void FTextHistory_AsPercent::Serialize(FStructuredArchive::FRecord Record)
{
	FTextHistory_FormatNumber::Serialize(Record);
}

bool FTextHistory_AsPercent::StaticShouldReadFromBuffer(const TCHAR* Buffer)
{
	return TEXT_STRINGIFICATION_PEEK_MARKER(TextStringificationUtil::LocGenPercentMarker);
}

const TCHAR* FTextHistory_AsPercent::ReadFromBuffer(const TCHAR* Buffer, const TCHAR* TextNamespace, const TCHAR* PackageNamespace)
{
	static const FString TokenMarker = TextStringificationUtil::LocGenPercentMarker;
	TEXT_STRINGIFICATION_FUNC_MODIFY_BUFFER_AND_VALIDATE(TextStringificationUtil::ReadNumberOrPercentFromBuffer, TokenMarker, SourceValue, FormatOptions, TargetCulture);
	MarkDisplayStringOutOfDate();
	return Buffer;
}

bool FTextHistory_AsPercent::WriteToBuffer(FString& Buffer, const bool bStripPackageNamespace) const
{
	TextStringificationUtil::WriteNumberOrPercentToBuffer(Buffer, TextStringificationUtil::LocGenPercentMarker, SourceValue, FormatOptions, TargetCulture, bStripPackageNamespace);
	return true;
}

bool FTextHistory_AsPercent::GetHistoricNumericData(const FText& InText, FHistoricTextNumericData& OutHistoricNumericData) const
{
	OutHistoricNumericData = FHistoricTextNumericData(FHistoricTextNumericData::EType::AsPercent, SourceValue, FormatOptions);
	return true;
}

///////////////////////////////////////
// FTextHistory_AsCurrency

FTextHistory_AsCurrency::FTextHistory_AsCurrency(FString&& InDisplayString, FFormatArgumentValue InSourceValue, FString InCurrencyCode, const FNumberFormattingOptions* const InFormatOptions, FCulturePtr InTargetCulture)
	: FTextHistory_FormatNumber(MoveTemp(InDisplayString), MoveTemp(InSourceValue), InFormatOptions, MoveTemp(InTargetCulture))
	, CurrencyCode(MoveTemp(InCurrencyCode))
{
}

FString FTextHistory_AsCurrency::BuildLocalizedDisplayString() const
{
	FInternationalization& I18N = FInternationalization::Get();
	checkf(I18N.IsInitialized() == true, TEXT("FInternationalization is not initialized. An FText formatting method was likely used in static object initialization - this is not supported."));
	const FCulture& Culture = TargetCulture.IsValid() ? *TargetCulture : *I18N.GetCurrentLocale();

	// when we remove AsCurrency should be easy to switch these to AsCurrencyBase and change SourceValue to be BaseVal in AsCurrencyBase (currently is the pre-divided value)
	const FDecimalNumberFormattingRules& FormattingRules = Culture.GetCurrencyFormattingRules(CurrencyCode);
	return BuildNumericDisplayString(FormattingRules);
}

FString FTextHistory_AsCurrency::BuildInvariantDisplayString() const
{
	FInternationalization& I18N = FInternationalization::Get();
	checkf(I18N.IsInitialized() == true, TEXT("FInternationalization is not initialized. An FText formatting method was likely used in static object initialization - this is not supported."));
	const FCulture& Culture = *I18N.GetInvariantCulture();

	// when we remove AsCurrency should be easy to switch these to AsCurrencyBase and change SourceValue to be BaseVal in AsCurrencyBase (currently is the pre-divided value)
	const FDecimalNumberFormattingRules& FormattingRules = Culture.GetCurrencyFormattingRules(CurrencyCode);
	return BuildNumericDisplayString(FormattingRules);
}

void FTextHistory_AsCurrency::Serialize(FStructuredArchive::FRecord Record)
{
	FArchive& BaseArchive = Record.GetUnderlyingArchive();

	if (BaseArchive.UEVer() >= VER_UE4_ADDED_CURRENCY_CODE_TO_FTEXT)
	{
		Record << SA_VALUE(TEXT("CurrencyCode"), CurrencyCode);
	}

	FTextHistory_FormatNumber::Serialize(Record);
}

bool FTextHistory_AsCurrency::StaticShouldReadFromBuffer(const TCHAR* Buffer)
{
	return TEXT_STRINGIFICATION_PEEK_MARKER(TextStringificationUtil::LocGenCurrencyMarker);
}

const TCHAR* FTextHistory_AsCurrency::ReadFromBuffer(const TCHAR* Buffer, const TCHAR* TextNamespace, const TCHAR* PackageNamespace)
{
	FInternationalization& I18N = FInternationalization::Get();
	checkf(I18N.IsInitialized() == true, TEXT("FInternationalization is not initialized. An FText formatting method was likely used in static object initialization - this is not supported."));
	const FCulture& Culture = TargetCulture.IsValid() ? *TargetCulture : *I18N.GetCurrentLocale();

	if (TEXT_STRINGIFICATION_PEEK_MARKER(TextStringificationUtil::LocGenCurrencyMarker))
	{
		// Parsing something of the form: LOCGEN_CURRENCY(..., "...", "...")
		TEXT_STRINGIFICATION_SKIP_MARKER_LEN(TextStringificationUtil::LocGenCurrencyMarker);

		// Skip whitespace before the opening bracket, and then step over it
		TEXT_STRINGIFICATION_SKIP_WHITESPACE_AND_CHAR('(');

		// Skip whitespace before the value, and then read out the number
		TEXT_STRINGIFICATION_SKIP_WHITESPACE();
		TEXT_STRINGIFICATION_READ_NUMBER(SourceValue);

		// Skip whitespace before the comma, and then step over it
		TEXT_STRINGIFICATION_SKIP_WHITESPACE_AND_CHAR(',');

		// Skip whitespace before the value, and then read out the quoted currency name
		TEXT_STRINGIFICATION_SKIP_WHITESPACE();
		TEXT_STRINGIFICATION_READ_QUOTED_STRING(CurrencyCode);

		// Skip whitespace before the comma, and then step over it
		TEXT_STRINGIFICATION_SKIP_WHITESPACE_AND_CHAR(',');

		// Skip whitespace before the value, and then read out the quoted culture name
		FString CultureNameString;
		TEXT_STRINGIFICATION_SKIP_WHITESPACE();
		TEXT_STRINGIFICATION_READ_QUOTED_STRING(CultureNameString);
		TargetCulture = (CultureNameString.IsEmpty()) ? nullptr : FInternationalization::Get().GetCulture(CultureNameString);

		// Skip whitespace before the closing bracket, and then step over it
		TEXT_STRINGIFICATION_SKIP_WHITESPACE_AND_CHAR(')');

		// Get the "base" value as a double
		double BaseValue = 0.0;
		switch (SourceValue.GetType())
		{
		case EFormatArgumentType::Int:
			BaseValue = (double)SourceValue.GetIntValue();
			break;
		case EFormatArgumentType::UInt:
			BaseValue = (double)SourceValue.GetUIntValue();
			break;
		case EFormatArgumentType::Float:
			BaseValue = SourceValue.GetFloatValue();
			break;
		case EFormatArgumentType::Double:
			BaseValue = SourceValue.GetDoubleValue();
			break;
		default:
			return nullptr;
		}

		// We need to convert the "base" value back to its pre-divided version
		const FDecimalNumberFormattingRules& FormattingRules = Culture.GetCurrencyFormattingRules(CurrencyCode);
		const FNumberFormattingOptions& FormattingOptions = FormattingRules.CultureDefaultFormattingOptions;
		SourceValue = BaseValue / static_cast<double>(FastDecimalFormat::Pow10(FormattingOptions.MaximumFractionalDigits));

		MarkDisplayStringOutOfDate();
		return Buffer;
	}

	return nullptr;
}

bool FTextHistory_AsCurrency::WriteToBuffer(FString& Buffer, const bool bStripPackageNamespace) const
{
	FInternationalization& I18N = FInternationalization::Get();
	checkf(I18N.IsInitialized() == true, TEXT("FInternationalization is not initialized. An FText formatting method was likely used in static object initialization - this is not supported."));
	const FCulture& Culture = TargetCulture.IsValid() ? *TargetCulture : *I18N.GetCurrentLocale();

	// Get the pre-divided value as a double
	double DividedValue = 0.0;
	switch (SourceValue.GetType())
	{
	case EFormatArgumentType::Int:
		DividedValue = (double)SourceValue.GetIntValue();
		break;
	case EFormatArgumentType::UInt:
		DividedValue = (double)SourceValue.GetUIntValue();
		break;
	case EFormatArgumentType::Float:
		DividedValue = SourceValue.GetFloatValue();
		break;
	case EFormatArgumentType::Double:
		DividedValue = SourceValue.GetDoubleValue();
		break;
	default:
		break;
	}

	// We need to convert the value back to its "base" version
	const FDecimalNumberFormattingRules& FormattingRules = Culture.GetCurrencyFormattingRules(CurrencyCode);
	const FNumberFormattingOptions& FormattingOptions = FormattingRules.CultureDefaultFormattingOptions;
	const int64 BaseVal = static_cast<int64>(DividedValue * static_cast<double>(FastDecimalFormat::Pow10(FormattingOptions.MaximumFractionalDigits)));

	// Produces LOCGEN_CURRENCY(..., "...", "...")
	Buffer += TEXT("LOCGEN_CURRENCY(");
	FFormatArgumentValue(BaseVal).ToExportedString(Buffer, bStripPackageNamespace);
	Buffer += TEXT(", \"");
	Buffer += CurrencyCode.ReplaceCharWithEscapedChar();
	Buffer += TEXT("\", \"");
	if (TargetCulture)
	{
		Buffer += TargetCulture->GetName().ReplaceCharWithEscapedChar();
	}
	Buffer += TEXT("\")");

	return true;
}

///////////////////////////////////////
// FTextHistory_AsDate

FTextHistory_AsDate::FTextHistory_AsDate(FString&& InDisplayString, FDateTime InSourceDateTime, const EDateTimeStyle::Type InDateStyle, FString InTimeZone, FCulturePtr InTargetCulture)
	: FTextHistory_Generated(MoveTemp(InDisplayString))
	, SourceDateTime(MoveTemp(InSourceDateTime))
	, DateStyle(InDateStyle)
	, TimeZone(MoveTemp(InTimeZone))
	, TargetCulture(MoveTemp(InTargetCulture))
{
	if (DateStyle == EDateTimeStyle::Custom)
	{
		DateStyle = EDateTimeStyle::Default;
	}
}

void FTextHistory_AsDate::Serialize(FStructuredArchive::FRecord Record)
{
	FTextHistory_Generated::Serialize(Record);

	FArchive& BaseArchive = Record.GetUnderlyingArchive();

	Record << SA_VALUE(TEXT("SourceDateTime"), SourceDateTime);

	int8 DateStyleInt8 = (int8)DateStyle;
	Record << SA_VALUE(TEXT("DateStyleInt8"), DateStyleInt8);
	DateStyle = (EDateTimeStyle::Type)DateStyleInt8;

	if( BaseArchive.UEVer() >= VER_UE4_FTEXT_HISTORY_DATE_TIMEZONE )
	{
		Record << SA_VALUE(TEXT("TimeZone"), TimeZone);
	}

	if(BaseArchive.IsSaving())
	{
		FString CultureName = TargetCulture.IsValid()? TargetCulture->GetName() : FString();
		Record << SA_VALUE(TEXT("CultureName"), CultureName);
	}
	else if(BaseArchive.IsLoading())
	{
		FString CultureName;
		Record << SA_VALUE(TEXT("CultureName"), CultureName);

		if(!CultureName.IsEmpty())
		{
			TargetCulture = FInternationalization::Get().GetCulture(CultureName);
		}
	}
}

bool FTextHistory_AsDate::StaticShouldReadFromBuffer(const TCHAR* Buffer)
{
	return TEXT_STRINGIFICATION_PEEK_MARKER(TextStringificationUtil::LocGenDateMarker);
}

const TCHAR* FTextHistory_AsDate::ReadFromBuffer(const TCHAR* Buffer, const TCHAR* TextNamespace, const TCHAR* PackageNamespace)
{
	static const FString TokenMarker = TextStringificationUtil::LocGenDateMarker;
	TEXT_STRINGIFICATION_FUNC_MODIFY_BUFFER_AND_VALIDATE(TextStringificationUtil::ReadDateTimeFromBuffer, TokenMarker, SourceDateTime, &DateStyle, nullptr, nullptr, TimeZone, TargetCulture);
	MarkDisplayStringOutOfDate();
	return Buffer;
}

bool FTextHistory_AsDate::WriteToBuffer(FString& Buffer, const bool bStripPackageNamespace) const
{
	TextStringificationUtil::WriteDateTimeToBuffer(Buffer, TextStringificationUtil::LocGenDateMarker, SourceDateTime, &DateStyle, nullptr, nullptr, TimeZone, TargetCulture, bStripPackageNamespace);
	return true;
}

bool FTextHistory_AsDate::IdenticalTo(const FTextHistory& Other, const ETextIdenticalModeFlags CompareModeFlags) const
{
	const FTextHistory_AsDate& CastOther = static_cast<const FTextHistory_AsDate&>(Other);

	return SourceDateTime == CastOther.SourceDateTime
		&& DateStyle == CastOther.DateStyle
		&& TimeZone == CastOther.TimeZone
		&& TargetCulture == CastOther.TargetCulture;
}

FString FTextHistory_AsDate::BuildLocalizedDisplayString() const
{
	FInternationalization& I18N = FInternationalization::Get();
	checkf(I18N.IsInitialized() == true, TEXT("FInternationalization is not initialized. An FText formatting method was likely used in static object initialization - this is not supported."));
	const FCulture& Culture = TargetCulture.IsValid() ? *TargetCulture : *I18N.GetCurrentLocale();

	return FTextChronoFormatter::AsDate(SourceDateTime, DateStyle, TimeZone, Culture);
}

FString FTextHistory_AsDate::BuildInvariantDisplayString() const
{
	FInternationalization& I18N = FInternationalization::Get();
	checkf(I18N.IsInitialized() == true, TEXT("FInternationalization is not initialized. An FText formatting method was likely used in static object initialization - this is not supported."));
	const FCulture& Culture = *I18N.GetInvariantCulture();

	return FTextChronoFormatter::AsDate(SourceDateTime, DateStyle, TimeZone, Culture);
}

///////////////////////////////////////
// FTextHistory_AsTime

FTextHistory_AsTime::FTextHistory_AsTime(FString&& InDisplayString, FDateTime InSourceDateTime, const EDateTimeStyle::Type InTimeStyle, FString InTimeZone, FCulturePtr InTargetCulture)
	: FTextHistory_Generated(MoveTemp(InDisplayString))
	, SourceDateTime(MoveTemp(InSourceDateTime))
	, TimeStyle(InTimeStyle)
	, TimeZone(MoveTemp(InTimeZone))
	, TargetCulture(MoveTemp(InTargetCulture))
{
	if (TimeStyle == EDateTimeStyle::Custom)
	{
		TimeStyle = EDateTimeStyle::Default;
	}
}

void FTextHistory_AsTime::Serialize(FStructuredArchive::FRecord Record)
{
	FTextHistory_Generated::Serialize(Record);

	FArchive& BaseArchive = Record.GetUnderlyingArchive();

	Record << SA_VALUE(TEXT("SourceDateTime"), SourceDateTime);

	int8 TimeStyleInt8 = (int8)TimeStyle;
	Record << SA_VALUE(TEXT("TimeStyle"), TimeStyleInt8);
	TimeStyle = (EDateTimeStyle::Type)TimeStyleInt8;

	Record << SA_VALUE(TEXT("TimeZone"), TimeZone);

	if(BaseArchive.IsSaving())
	{
		FString CultureName = TargetCulture.IsValid()? TargetCulture->GetName() : FString();
		Record << SA_VALUE(TEXT("CultureName"), CultureName);
	}
	else if(BaseArchive.IsLoading())
	{
		FString CultureName;
		Record << SA_VALUE(TEXT("CultureName"), CultureName);

		if(!CultureName.IsEmpty())
		{
			TargetCulture = FInternationalization::Get().GetCulture(CultureName);
		}
	}
}

bool FTextHistory_AsTime::StaticShouldReadFromBuffer(const TCHAR* Buffer)
{
	return TEXT_STRINGIFICATION_PEEK_MARKER(TextStringificationUtil::LocGenTimeMarker);
}

const TCHAR* FTextHistory_AsTime::ReadFromBuffer(const TCHAR* Buffer, const TCHAR* TextNamespace, const TCHAR* PackageNamespace)
{
	static const FString TokenMarker = TextStringificationUtil::LocGenTimeMarker;
	TEXT_STRINGIFICATION_FUNC_MODIFY_BUFFER_AND_VALIDATE(TextStringificationUtil::ReadDateTimeFromBuffer, TokenMarker, SourceDateTime, nullptr, &TimeStyle, nullptr, TimeZone, TargetCulture);
	MarkDisplayStringOutOfDate();
	return Buffer;
}

bool FTextHistory_AsTime::WriteToBuffer(FString& Buffer, const bool bStripPackageNamespace) const
{
	TextStringificationUtil::WriteDateTimeToBuffer(Buffer, TextStringificationUtil::LocGenTimeMarker, SourceDateTime, nullptr, &TimeStyle, nullptr, TimeZone, TargetCulture, bStripPackageNamespace);
	return true;
}

bool FTextHistory_AsTime::IdenticalTo(const FTextHistory& Other, const ETextIdenticalModeFlags CompareModeFlags) const
{
	const FTextHistory_AsTime& CastOther = static_cast<const FTextHistory_AsTime&>(Other);
	
	return SourceDateTime == CastOther.SourceDateTime
		&& TimeStyle == CastOther.TimeStyle
		&& TimeZone == CastOther.TimeZone
		&& TargetCulture == CastOther.TargetCulture;
}

FString FTextHistory_AsTime::BuildLocalizedDisplayString() const
{
	FInternationalization& I18N = FInternationalization::Get();
	checkf(I18N.IsInitialized() == true, TEXT("FInternationalization is not initialized. An FText formatting method was likely used in static object initialization - this is not supported."));
	const FCulture& Culture = TargetCulture.IsValid() ? *TargetCulture : *I18N.GetCurrentLocale();

	return FTextChronoFormatter::AsTime(SourceDateTime, TimeStyle, TimeZone, Culture);
}

FString FTextHistory_AsTime::BuildInvariantDisplayString() const
{
	FInternationalization& I18N = FInternationalization::Get();
	checkf(I18N.IsInitialized() == true, TEXT("FInternationalization is not initialized. An FText formatting method was likely used in static object initialization - this is not supported."));
	const FCulture& Culture = *I18N.GetInvariantCulture();

	return FTextChronoFormatter::AsTime(SourceDateTime, TimeStyle, TimeZone, Culture);
}

///////////////////////////////////////
// FTextHistory_AsDateTime

FTextHistory_AsDateTime::FTextHistory_AsDateTime(FString&& InDisplayString, FDateTime InSourceDateTime, const EDateTimeStyle::Type InDateStyle, const EDateTimeStyle::Type InTimeStyle, FString InTimeZone, FCulturePtr InTargetCulture)
	: FTextHistory_Generated(MoveTemp(InDisplayString))
	, SourceDateTime(MoveTemp(InSourceDateTime))
	, DateStyle(InDateStyle)
	, TimeStyle(InTimeStyle)
	, CustomPattern()
	, TimeZone(MoveTemp(InTimeZone))
	, TargetCulture(MoveTemp(InTargetCulture))
{
	if (DateStyle == EDateTimeStyle::Custom)
	{
		DateStyle = EDateTimeStyle::Default;
	}
	if (TimeStyle == EDateTimeStyle::Custom)
	{
		TimeStyle = EDateTimeStyle::Default;
	}
}

FTextHistory_AsDateTime::FTextHistory_AsDateTime(FString&& InDisplayString, FDateTime InSourceDateTime, FString InCustomPattern, FString InTimeZone, FCulturePtr InTargetCulture)
	: FTextHistory_Generated(MoveTemp(InDisplayString))
	, SourceDateTime(MoveTemp(InSourceDateTime))
	, DateStyle(EDateTimeStyle::Custom)
	, TimeStyle(EDateTimeStyle::Custom)
	, CustomPattern(MoveTemp(InCustomPattern))
	, TimeZone(MoveTemp(InTimeZone))
	, TargetCulture(MoveTemp(InTargetCulture))
{
}

void FTextHistory_AsDateTime::Serialize(FStructuredArchive::FRecord Record)
{
	FTextHistory_Generated::Serialize(Record);

	FArchive& BaseArchive = Record.GetUnderlyingArchive();

	Record << SA_VALUE(TEXT("SourceDateTime"), SourceDateTime);

	int8 DateStyleInt8 = (int8)DateStyle;
	Record << SA_VALUE(TEXT("DateStyle"), DateStyleInt8);
	DateStyle = (EDateTimeStyle::Type)DateStyleInt8;

	int8 TimeStyleInt8 = (int8)TimeStyle;
	Record << SA_VALUE(TEXT("TimeStyle"), TimeStyleInt8);
	TimeStyle = (EDateTimeStyle::Type)TimeStyleInt8;

	if (DateStyle == EDateTimeStyle::Custom)
	{
		Record << SA_VALUE(TEXT("CustomPattern"), CustomPattern);
	}

	Record << SA_VALUE(TEXT("TimeZone"), TimeZone);

	if(BaseArchive.IsSaving())
	{
		FString CultureName = TargetCulture.IsValid()? TargetCulture->GetName() : FString();
		Record << SA_VALUE(TEXT("CultureName"), CultureName);
	}
	else if(BaseArchive.IsLoading())
	{
		FString CultureName;
		Record << SA_VALUE(TEXT("CultureName"), CultureName);

		if(!CultureName.IsEmpty())
		{
			TargetCulture = FInternationalization::Get().GetCulture(CultureName);
		}
	}
}

bool FTextHistory_AsDateTime::StaticShouldReadFromBuffer(const TCHAR* Buffer)
{
	return TEXT_STRINGIFICATION_PEEK_MARKER(TextStringificationUtil::LocGenDateTimeMarker);
}

const TCHAR* FTextHistory_AsDateTime::ReadFromBuffer(const TCHAR* Buffer, const TCHAR* TextNamespace, const TCHAR* PackageNamespace)
{
	static const FString TokenMarker = TextStringificationUtil::LocGenDateTimeMarker;
	TEXT_STRINGIFICATION_FUNC_MODIFY_BUFFER_AND_VALIDATE(TextStringificationUtil::ReadDateTimeFromBuffer, TokenMarker, SourceDateTime, &DateStyle, &TimeStyle, &CustomPattern, TimeZone, TargetCulture);
	MarkDisplayStringOutOfDate();
	return Buffer;
}

bool FTextHistory_AsDateTime::WriteToBuffer(FString& Buffer, const bool bStripPackageNamespace) const
{
	TextStringificationUtil::WriteDateTimeToBuffer(Buffer, TextStringificationUtil::LocGenDateTimeMarker, SourceDateTime, &DateStyle, &TimeStyle, &CustomPattern, TimeZone, TargetCulture, bStripPackageNamespace);
	return true;
}

bool FTextHistory_AsDateTime::IdenticalTo(const FTextHistory& Other, const ETextIdenticalModeFlags CompareModeFlags) const
{
	const FTextHistory_AsDateTime& CastOther = static_cast<const FTextHistory_AsDateTime&>(Other);
	
	return SourceDateTime == CastOther.SourceDateTime
		&& DateStyle == CastOther.DateStyle
		&& TimeStyle == CastOther.TimeStyle
		&& CustomPattern == CastOther.CustomPattern
		&& TimeZone == CastOther.TimeZone
		&& TargetCulture == CastOther.TargetCulture;
}

FString FTextHistory_AsDateTime::BuildLocalizedDisplayString() const
{
	FInternationalization& I18N = FInternationalization::Get();
	checkf(I18N.IsInitialized() == true, TEXT("FInternationalization is not initialized. An FText formatting method was likely used in static object initialization - this is not supported."));
	const FCulture& Culture = TargetCulture.IsValid() ? *TargetCulture : *I18N.GetCurrentLocale();

	return DateStyle == EDateTimeStyle::Custom
		? FTextChronoFormatter::AsDateTime(SourceDateTime, CustomPattern, TimeZone, Culture)
		: FTextChronoFormatter::AsDateTime(SourceDateTime, DateStyle, TimeStyle, TimeZone, Culture);
}

FString FTextHistory_AsDateTime::BuildInvariantDisplayString() const
{
	FInternationalization& I18N = FInternationalization::Get();
	checkf(I18N.IsInitialized() == true, TEXT("FInternationalization is not initialized. An FText formatting method was likely used in static object initialization - this is not supported."));
	const FCulture& Culture = *I18N.GetInvariantCulture();

	return DateStyle == EDateTimeStyle::Custom
		? FTextChronoFormatter::AsDateTime(SourceDateTime, CustomPattern, TimeZone, Culture)
		: FTextChronoFormatter::AsDateTime(SourceDateTime, DateStyle, TimeStyle, TimeZone, Culture);
}

///////////////////////////////////////
// FTextHistory_Transform

FTextHistory_Transform::FTextHistory_Transform(FString&& InDisplayString, FText InSourceText, const ETransformType InTransformType)
	: FTextHistory_Generated(MoveTemp(InDisplayString))
	, SourceText(MoveTemp(InSourceText))
	, TransformType(InTransformType)
{
}

void FTextHistory_Transform::Serialize(FStructuredArchive::FRecord Record)
{
	FTextHistory_Generated::Serialize(Record);

	FArchive& BaseArchive = Record.GetUnderlyingArchive();

	Record << SA_VALUE(TEXT("SourceText"), SourceText);

	uint8& TransformTypeRef = (uint8&)TransformType;
	Record << SA_VALUE(TEXT("TransformType"), TransformTypeRef);
}

bool FTextHistory_Transform::StaticShouldReadFromBuffer(const TCHAR* Buffer)
{
	return TEXT_STRINGIFICATION_PEEK_MARKER(TextStringificationUtil::LocGenToLowerMarker)
		|| TEXT_STRINGIFICATION_PEEK_MARKER(TextStringificationUtil::LocGenToUpperMarker);
}

const TCHAR* FTextHistory_Transform::ReadFromBuffer(const TCHAR* Buffer, const TCHAR* TextNamespace, const TCHAR* PackageNamespace)
{
	// Parsing something of the form: LOCGEN_TOLOWER(...) or LOCGEN_TOUPPER
	if (TEXT_STRINGIFICATION_PEEK_MARKER(TextStringificationUtil::LocGenToLowerMarker))
	{
		TEXT_STRINGIFICATION_SKIP_MARKER_LEN(TextStringificationUtil::LocGenToLowerMarker);
		TransformType = ETransformType::ToLower;
	}
	else if (TEXT_STRINGIFICATION_PEEK_MARKER(TextStringificationUtil::LocGenToUpperMarker))
	{
		TEXT_STRINGIFICATION_SKIP_MARKER_LEN(TextStringificationUtil::LocGenToUpperMarker);
		TransformType = ETransformType::ToUpper;
	}
	else
	{
		return nullptr;
	}

	// Skip whitespace before the opening bracket, and then step over it
	TEXT_STRINGIFICATION_SKIP_WHITESPACE_AND_CHAR('(');

	// Skip whitespace before the value, and then read out the text
	TEXT_STRINGIFICATION_SKIP_WHITESPACE();
	TEXT_STRINGIFICATION_FUNC_MODIFY_BUFFER_AND_VALIDATE(FTextStringHelper::ReadFromBuffer, SourceText, nullptr, nullptr, true);

	// Skip whitespace before the closing bracket, and then step over it
	TEXT_STRINGIFICATION_SKIP_WHITESPACE_AND_CHAR(')');

	MarkDisplayStringOutOfDate();
	return Buffer;
}

bool FTextHistory_Transform::WriteToBuffer(FString& Buffer, const bool bStripPackageNamespace) const
{
	// Produces LOCGEN_TOLOWER(...) or LOCGEN_TOUPPER
	switch (TransformType)
	{
	case ETransformType::ToLower:
		Buffer += TEXT("LOCGEN_TOLOWER(");
		break;
	case ETransformType::ToUpper:
		Buffer += TEXT("LOCGEN_TOUPPER(");
		break;
	default:
		break;
	}
	FTextStringHelper::WriteToBuffer(Buffer, SourceText, /*bRequireQuotes*/true, bStripPackageNamespace);
	Buffer += TEXT(")");

	return true;
}

bool FTextHistory_Transform::IdenticalTo(const FTextHistory& Other, const ETextIdenticalModeFlags CompareModeFlags) const
{
	const FTextHistory_Transform& CastOther = static_cast<const FTextHistory_Transform&>(Other);
	
	return SourceText.IdenticalTo(CastOther.SourceText, CompareModeFlags)
		&& TransformType == CastOther.TransformType;
}

FString FTextHistory_Transform::BuildLocalizedDisplayString() const
{
	SourceText.Rebuild();

	switch (TransformType)
	{
	case ETransformType::ToLower:
		return FTextTransformer::ToLower(SourceText.ToString());
	case ETransformType::ToUpper:
		return FTextTransformer::ToUpper(SourceText.ToString());
	default:
		break;
	}
	return FString();
}

FString FTextHistory_Transform::BuildInvariantDisplayString() const
{
	SourceText.Rebuild();

	switch (TransformType)
	{
	case ETransformType::ToLower:
		return FTextTransformer::ToLower(SourceText.BuildSourceString());
	case ETransformType::ToUpper:
		return FTextTransformer::ToUpper(SourceText.BuildSourceString());
	default:
		break;
	}
	return FString();
}

void FTextHistory_Transform::GetHistoricFormatData(const FText& InText, TArray<FHistoricTextFormatData>& OutHistoricFormatData) const
{
	FTextInspector::GetHistoricFormatData(SourceText, OutHistoricFormatData);
}

bool FTextHistory_Transform::GetHistoricNumericData(const FText& InText, FHistoricTextNumericData& OutHistoricNumericData) const
{
	return FTextInspector::GetHistoricNumericData(SourceText, OutHistoricNumericData);
}

#if WITH_EDITORONLY_DATA
const FString& FTextHistory_Transform::GetDevNotes() const
{
	return SourceText.GetDevNotes();
}
#endif

///////////////////////////////////////
// FTextHistory_StringTableEntry

FTextHistory_StringTableEntry::FTextHistory_StringTableEntry(FName InTableId, const FTextKey& InKey, const EStringTableLoadingPolicy InLoadingPolicy)
	: StringTableReferenceData(MakeShared<FStringTableReferenceData, ESPMode::ThreadSafe>())
{
	StringTableReferenceData->Initialize(InTableId, InKey, InLoadingPolicy);
	MarkDisplayStringUpToDate();
}

FTextId FTextHistory_StringTableEntry::GetTextId() const
{
	if (StringTableReferenceData)
	{
		return StringTableReferenceData->GetTextId();
	}
	return FTextId();
}

FTextConstDisplayStringPtr FTextHistory_StringTableEntry::GetLocalizedString() const
{
	return StringTableReferenceData ? StringTableReferenceData->ResolveDisplayString() : nullptr;
}

const FString& FTextHistory_StringTableEntry::GetSourceString() const
{
	FStringTableEntryConstPtr StringTableEntryPin = StringTableReferenceData ? StringTableReferenceData->ResolveStringTableEntry() : nullptr;
	if (StringTableEntryPin.IsValid())
	{
		return StringTableEntryPin->GetSourceString();
	}
	return FStringTableEntry::GetPlaceholderSourceString();
}

const FString& FTextHistory_StringTableEntry::GetDisplayString() const
{
	if (FTextLocalizationManager::IsDisplayStringSupportEnabled())
	{
		if (FTextConstDisplayStringPtr DisplayString = GetLocalizedString())
		{
			return *DisplayString;
		}
	}
	return GetSourceString();
}

FString FTextHistory_StringTableEntry::BuildInvariantDisplayString() const
{
	return GetSourceString();
}

bool FTextHistory_StringTableEntry::IdenticalTo(const FTextHistory& Other, const ETextIdenticalModeFlags CompareModeFlags) const
{
	const FTextHistory_StringTableEntry& CastOther = static_cast<const FTextHistory_StringTableEntry&>(Other);

	return StringTableReferenceData->IsIdentical(*CastOther.StringTableReferenceData);
}

void FTextHistory_StringTableEntry::GetTableIdAndKey(FName& OutTableId, FTextKey& OutKey) const
{
	if (StringTableReferenceData)
	{
		StringTableReferenceData->GetTableIdAndKey(OutTableId, OutKey);
	}
}

void FTextHistory_StringTableEntry::Serialize(FStructuredArchive::FRecord Record)
{
	FArchive& BaseArchive = Record.GetUnderlyingArchive();

	if (BaseArchive.IsLoading())
	{
		FName TableId;
		FTextKey Key;
		Record << SA_VALUE(TEXT("TableId"), TableId);
		Key.SerializeAsString(Record.EnterField(TEXT("Key")));

		StringTableReferenceData = MakeShared<FStringTableReferenceData, ESPMode::ThreadSafe>();
		StringTableReferenceData->Initialize(TableId, Key, EStringTableLoadingPolicy::FindOrLoad);
		MarkDisplayStringUpToDate();
	}
	else if (BaseArchive.IsSaving())
	{
		FName TableId;
		FTextKey Key;
		if (StringTableReferenceData)
		{
			StringTableReferenceData->GetTableIdAndKey(TableId, Key);
		}

		Record << SA_VALUE(TEXT("TableId"), TableId);
		Key.SerializeAsString(Record.EnterField(TEXT("Key")));
	}

	// Collect string table asset references
	if (StringTableReferenceData)
	{
		StringTableReferenceData->CollectStringTableAssetReferences(Record);
	}
}

void FTextHistory_StringTableEntry::UpdateDisplayString()
{
	if (StringTableReferenceData)
	{
		StringTableReferenceData->ResolveDisplayString(/*bForceRefresh*/true);
	}
}

#if WITH_EDITORONLY_DATA
const FString& FTextHistory_StringTableEntry::GetDevNotes() const
{
	if (StringTableReferenceData)
	{
		if (const FStringTableEntryConstPtr StringTableEntryPin = StringTableReferenceData->ResolveStringTableEntry())
		{
			return StringTableEntryPin->GetDevNotes();
		}
	}
	
	return FTextHistory::GetDevNotes();
}
#endif

bool FTextHistory_StringTableEntry::StaticShouldReadFromBuffer(const TCHAR* Buffer)
{
	return TEXT_STRINGIFICATION_PEEK_MARKER(TextStringificationUtil::LocTableMarker);
}

const TCHAR* FTextHistory_StringTableEntry::ReadFromBuffer(const TCHAR* Buffer, const TCHAR* TextNamespace, const TCHAR* PackageNamespace)
{
#define LOC_DEFINE_REGION
	if (TEXT_STRINGIFICATION_PEEK_MARKER(TextStringificationUtil::LocTableMarker))
	{
		// Parsing something of the form: LOCTABLE("...", "...")
		TEXT_STRINGIFICATION_SKIP_MARKER_LEN(TextStringificationUtil::LocTableMarker);

		// Skip whitespace before the opening bracket, and then step over it
		TEXT_STRINGIFICATION_SKIP_WHITESPACE_AND_CHAR('(');

		// Skip whitespace before the value, and then read out the quoted table ID
		FString TableIdString;
		TEXT_STRINGIFICATION_SKIP_WHITESPACE();
		TEXT_STRINGIFICATION_READ_QUOTED_STRING(TableIdString);
		FName TableId = *TableIdString;

		// Skip whitespace before the comma, and then step over it
		TEXT_STRINGIFICATION_SKIP_WHITESPACE_AND_CHAR(',');

		// Skip whitespace before the value, and then read out the quoted key
		FString Key;
		TEXT_STRINGIFICATION_SKIP_WHITESPACE();
		TEXT_STRINGIFICATION_READ_QUOTED_STRING(Key);

		// Skip whitespace before the closing bracket, and then step over it
		TEXT_STRINGIFICATION_SKIP_WHITESPACE_AND_CHAR(')');

		// Prepare the string table reference
		StringTableReferenceData = MakeShared<FStringTableReferenceData, ESPMode::ThreadSafe>();
		StringTableReferenceData->Initialize(TableId, MoveTemp(Key), EStringTableLoadingPolicy::FindOrLoad);
		MarkDisplayStringUpToDate();

		return Buffer;
	}
#undef LOC_DEFINE_REGION

	return nullptr;
}

bool FTextHistory_StringTableEntry::WriteToBuffer(FString& Buffer, const bool bStripPackageNamespace) const
{
	if (StringTableReferenceData)
	{
		FName TableId;
		FTextKey Key;
		StringTableReferenceData->GetTableIdAndKey(TableId, Key);

		FString KeyStr = Key.ToString();

#define LOC_DEFINE_REGION
		// Produces LOCTABLE("...", "...")
		Buffer += TEXT("LOCTABLE(\"");
		Buffer += TableId.ToString().ReplaceCharWithEscapedChar();
		Buffer += TEXT("\", \"");
		Buffer += KeyStr.ReplaceCharWithEscapedChar();
		Buffer += TEXT("\")");
#undef LOC_DEFINE_REGION

		return true;
	}

	return false;
}

void FTextHistory_StringTableEntry::FStringTableReferenceData::Initialize(FName InTableId, const FTextKey& InKey, const EStringTableLoadingPolicy InLoadingPolicy)
{
	TableId = InTableId;
	Key = InKey;
	FStringTableRedirects::RedirectTableIdAndKey(TableId, Key);

	if (InLoadingPolicy == EStringTableLoadingPolicy::Find)
	{
		// No loading attempt
		LoadingPhase = EStringTableLoadingPhase::Loaded;
		ResolveDisplayString();
	}
	else if (InLoadingPolicy == EStringTableLoadingPolicy::FindOrFullyLoad && IStringTableEngineBridge::CanFindOrLoadStringTableAsset())
	{
		// Forced synchronous load
		LoadingPhase = EStringTableLoadingPhase::Loaded;
		IStringTableEngineBridge::FullyLoadStringTableAsset(TableId);
		ResolveDisplayString();
	}
	else
	{
		// Potential asynchronous load
		LoadingPhase = EStringTableLoadingPhase::PendingLoad;
		ConditionalBeginAssetLoad();
	}
}

bool FTextHistory_StringTableEntry::FStringTableReferenceData::IsIdentical(const FStringTableReferenceData& Other) const
{
	UE::TScopeLock ScopeLock(DataCS);
	UE::TScopeLock OtherScopeLock(Other.DataCS);

	return TableId == Other.TableId
		&& Key == Other.Key;
}

FName FTextHistory_StringTableEntry::FStringTableReferenceData::GetTableId() const
{
	UE::TScopeLock ScopeLock(DataCS);
	return TableId;
}

FTextKey FTextHistory_StringTableEntry::FStringTableReferenceData::GetKey() const
{
	UE::TScopeLock ScopeLock(DataCS);
	return Key;
}

void FTextHistory_StringTableEntry::FStringTableReferenceData::GetTableIdAndKey(FName& OutTableId, FTextKey& OutKey) const
{
	UE::TScopeLock ScopeLock(DataCS);
	OutTableId = TableId;
	OutKey = Key;
}

FTextId FTextHistory_StringTableEntry::FStringTableReferenceData::GetTextId()
{
	if (FStringTableEntryConstPtr StringTableEntryPin = ResolveStringTableEntry())
	{
		return StringTableEntryPin->GetDisplayStringId();
	}
	return FTextId();
}

void FTextHistory_StringTableEntry::FStringTableReferenceData::CollectStringTableAssetReferences(FStructuredArchive::FRecord Record)
{
	FArchive& BaseArchive = Record.GetUnderlyingArchive();

	if (BaseArchive.IsObjectReferenceCollector())
	{
		UE::TScopeLock ScopeLock(DataCS);

		const FName OldTableId = TableId;
		IStringTableEngineBridge::CollectStringTableAssetReferences(TableId, Record.EnterField(TEXT("AssetReferences")));

		if (TableId != OldTableId)
		{
			// This String Table asset was redirected, so we'll need to re-resolve the String Table entry later
			StringTableEntry.Reset();
			DisplayString.Reset();
		}
	}

	IStringTableEngineBridge::CollectStringTableAssetSearchableName(TableId, Key, BaseArchive);
}

FStringTableEntryConstPtr FTextHistory_StringTableEntry::FStringTableReferenceData::ResolveStringTableEntry()
{
	FStringTableEntryConstPtr StringTableEntryPin = StringTableEntry.Pin();

	if (!StringTableEntryPin.IsValid())
	{
		ConditionalBeginAssetLoad();
	}

	if (!StringTableEntryPin.IsValid() || !StringTableEntryPin->IsOwned())
	{
		UE::TScopeLock ScopeLock(DataCS);

		// Reset for the case it was disowned rather than became null
		StringTableEntry.Reset();
		StringTableEntryPin.Reset();
		DisplayString.Reset();

		if (LoadingPhase != EStringTableLoadingPhase::Loaded)
		{
			// Table still loading - cannot be resolved yet
			return nullptr;
		}

		FStringTableConstPtr StringTable = FStringTableRegistry::Get().FindStringTable(TableId);
		if (StringTable.IsValid())
		{
			if (!StringTable->IsLoaded())
			{
				// Table still loading - cannot be resolved yet
				return nullptr;
			}
			StringTableEntryPin = StringTable->FindEntry(Key);
		}
		
		StringTableEntry = StringTableEntryPin;
	}

	if (!StringTableEntryPin.IsValid())
	{
		FStringTableRegistry::Get().LogMissingStringTableEntry(TableId, Key);
	}

	return StringTableEntryPin;
}

FTextConstDisplayStringPtr FTextHistory_StringTableEntry::FStringTableReferenceData::ResolveDisplayString(const bool bForceRefresh)
{
	FStringTableEntryConstPtr StringTableEntryPin = ResolveStringTableEntry();

	if (StringTableEntryPin && (!DisplayString || bForceRefresh))
	{
		DisplayString = StringTableEntryPin->GetDisplayString();
	}

	return DisplayString;
}

void FTextHistory_StringTableEntry::FStringTableReferenceData::ConditionalBeginAssetLoad()
{
	if (!IStringTableEngineBridge::CanFindOrLoadStringTableAsset())
	{
		return;
	}

	FName TableIdToLoad;
	{
		UE::TScopeLock ScopeLock(DataCS);

		if (LoadingPhase != EStringTableLoadingPhase::PendingLoad)
		{
			return;
		}

		TableIdToLoad = TableId;
		LoadingPhase = EStringTableLoadingPhase::Loading;
	}

	IStringTableEngineBridge::LoadStringTableAsset(TableIdToLoad, [WeakThis = FStringTableReferenceDataWeakPtr(AsShared())](FName InRequestedTableId, FName InLoadedTableId)
	{
		// Was this request still valid?
		FStringTableReferenceDataPtr This = WeakThis.Pin();
		if (!This)
		{
			return;
		}

		UE::TScopeLock ScopeLock(This->DataCS);
		check(This->TableId == InRequestedTableId);

		// If this string table loaded, then update the table ID using the potentially redirected value
		if (!InLoadedTableId.IsNone())
		{
			This->TableId = InLoadedTableId;
		}
		This->LoadingPhase = EStringTableLoadingPhase::Loaded;

		This->ResolveDisplayString();
	});
}


///////////////////////////////////////
// FTextHistory_TextGenerator

FTextHistory_TextGenerator::FTextHistory_TextGenerator(FString&& InDisplayString, const TSharedRef<ITextGenerator>& InTextGenerator)
	: FTextHistory_Generated(MoveTemp(InDisplayString))
	, TextGenerator(InTextGenerator)
{
}

bool FTextHistory_TextGenerator::IdenticalTo(const FTextHistory& Other, const ETextIdenticalModeFlags CompareModeFlags) const
{
	const FTextHistory_TextGenerator& CastOther = static_cast<const FTextHistory_TextGenerator&>(Other);
	// TODO: Could add this to the ITextGenerator API
	return false;
}

FString FTextHistory_TextGenerator::BuildLocalizedDisplayString() const
{
	return TextGenerator
		? TextGenerator->BuildLocalizedDisplayString()
		: FString();
}

FString FTextHistory_TextGenerator::BuildInvariantDisplayString() const
{
	return TextGenerator
		? TextGenerator->BuildInvariantDisplayString()
		: FString();
}

void FTextHistory_TextGenerator::Serialize(FStructuredArchive::FRecord Record)
{
	FTextHistory_Generated::Serialize(Record);

	FArchive& BaseArchive = Record.GetUnderlyingArchive();
	
	FName GeneratorTypeID = (BaseArchive.IsSaving() && TextGenerator)
		? TextGenerator->GetTypeID()
		: FName();
	Record << SA_VALUE(TEXT("GeneratorTypeID"), GeneratorTypeID);

	TArray<uint8> GeneratorContents;

	if (BaseArchive.IsLoading())
	{
		TextGenerator.Reset();

		// Look up and construct or skip
		if (GeneratorTypeID != NAME_None)
		{
			FText::FCreateTextGeneratorDelegate FactoryFunction = FText::FindRegisteredTextGenerator(GeneratorTypeID);
			Record << SA_VALUE(TEXT("GeneratorContents"), GeneratorContents);

			if (ensureMsgf(FactoryFunction.IsBound(), TEXT("FTextHistory_TextGenerator::Serialize(): Unable to find registered text generator for \"%s\". Use FText::RegisterTextGenerator() to register a handler."),
				*GeneratorTypeID.ToString()))
			{
				FMemoryReader ArReader(GeneratorContents);
				FStructuredArchiveFromArchive ArStructuredReader(ArReader);

				{
					FStructuredArchive::FRecord ContentRecord = ArStructuredReader.GetSlot().EnterRecord();
					TextGenerator = FactoryFunction.Execute(ContentRecord);
					TextGenerator->Serialize(ContentRecord);
				}

				if (ArReader.IsError())
				{
					BaseArchive.SetError();
				}
			}
		}

		MarkDisplayStringOutOfDate();
	}
	else if (BaseArchive.IsSaving())
	{
		if (ensureMsgf(GeneratorTypeID != NAME_None, TEXT("FTextHistory_TextGenerator::Serialize(): Attempting to serialize a generator type that is not serializable")))
		{
			ensureMsgf(
				FText::FindRegisteredTextGenerator(GeneratorTypeID).IsBound(),
				TEXT("FTextHistory_TextGenerator::Serialize(): No generator factory function is registered for type \"%s\". Deserialization will fail. Use FText::RegisterTextGenerator() to register a handler."),
				*GeneratorTypeID.ToString()
			);

			FMemoryWriter ArWriter(GeneratorContents);
			FStructuredArchiveFromArchive ArStructuredWriter(ArWriter);

			TextGenerator->Serialize(ArStructuredWriter.GetSlot().EnterRecord());
			Record << SA_VALUE(TEXT("GeneratorContents"), GeneratorContents);

			if (ArWriter.IsError())
			{
				BaseArchive.SetError();
			}
		}
	}
}
