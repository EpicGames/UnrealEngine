// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Async/TransactionallySafeMutex.h"
#include "Containers/Array.h"
#include "CoreTypes.h"
#include "HAL/CriticalSection.h"
#include "Internationalization/CulturePointer.h"
#include "Internationalization/ITextData.h"
#include "Internationalization/LocalizedTextSourceTypes.h"
#include "Internationalization/StringTableCoreFwd.h"
#include "Internationalization/Text.h"
#include "Internationalization/TextKey.h"
#include "Misc/DateTime.h"
#include "Misc/TransactionallySafeCriticalSection.h"
#include "Serialization/StructuredArchive.h"
#include "Templates/SharedPointerInternals.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/NameTypes.h"

class ITextGenerator;
struct FDecimalNumberFormattingRules;

enum class ETextHistoryType : int8
{
	None = -1,
	Base = 0,
	NamedFormat,
	OrderedFormat,
	ArgumentFormat,
	AsNumber,
	AsPercent,
	AsCurrency,
	AsDate,
	AsTime,
	AsDateTime,
	Transform,
	StringTableEntry,
	TextGenerator,

	// Add new enum types at the end only! They are serialized by index.
};

#define OVERRIDE_TEXT_HISTORY_STRINGIFICATION																						\
	static  bool StaticShouldReadFromBuffer(const TCHAR* Buffer);																	\
	virtual bool ShouldReadFromBuffer(const TCHAR* Buffer) const override { return StaticShouldReadFromBuffer(Buffer); }			\
	virtual const TCHAR* ReadFromBuffer(const TCHAR* Buffer, const TCHAR* TextNamespace, const TCHAR* PackageNamespace) override;	\
	virtual bool WriteToBuffer(FString& Buffer, const bool bStripPackageNamespace) const override;

/** Utilities for stringifying text */
namespace TextStringificationUtil
{

#define LOC_DEFINE_REGION
inline const auto& TextMarker = TEXT("TEXT");
inline const auto& InvTextMarker = TEXT("INVTEXT");
inline const auto& NsLocTextMarker = TEXT("NSLOCTEXT");
inline const auto& LocTextMarker = TEXT("LOCTEXT");
inline const auto& NsNoteLocTextMarker = TEXT("NSNOTELOCTEXT");
inline const auto& NoteLocTextMarker = TEXT("NOTELOCTEXT");
inline const auto& LocTableMarker = TEXT("LOCTABLE");
inline const auto& LocGenNumberMarker = TEXT("LOCGEN_NUMBER");
inline const auto& LocGenPercentMarker = TEXT("LOCGEN_PERCENT");
inline const auto& LocGenCurrencyMarker = TEXT("LOCGEN_CURRENCY");
inline const auto& LocGenDateMarker = TEXT("LOCGEN_DATE");
inline const auto& LocGenTimeMarker = TEXT("LOCGEN_TIME");
inline const auto& LocGenDateTimeMarker = TEXT("LOCGEN_DATETIME");
inline const auto& LocGenToLowerMarker = TEXT("LOCGEN_TOLOWER");
inline const auto& LocGenToUpperMarker = TEXT("LOCGEN_TOUPPER");
inline const auto& LocGenFormatOrderedMarker = TEXT("LOCGEN_FORMAT_ORDERED");
inline const auto& LocGenFormatNamedMarker = TEXT("LOCGEN_FORMAT_NAMED");
inline const auto& GroupedSuffix = TEXT("_GROUPED");
inline const auto& UngroupedSuffix = TEXT("_UNGROUPED");
inline const auto& CustomSuffix = TEXT("_CUSTOM");
inline const auto& UtcSuffix = TEXT("_UTC");
inline const auto& LocalSuffix = TEXT("_LOCAL");
#undef LOC_DEFINE_REGION

#define TEXT_STRINGIFICATION_FUNC_MODIFY_BUFFER_AND_VALIDATE(Func, ...)		\
	Buffer = Func(Buffer, ##__VA_ARGS__);									\
	if (!Buffer) { return nullptr; }

#define TEXT_STRINGIFICATION_PEEK_MARKER(T)					TextStringificationUtil::PeekMarker(Buffer, T, UE_ARRAY_COUNT(T) - 1)
#define TEXT_STRINGIFICATION_PEEK_INSENSITIVE_MARKER(T)		TextStringificationUtil::PeekInsensitiveMarker(Buffer, T, UE_ARRAY_COUNT(T) - 1)
bool PeekMarker(const TCHAR* Buffer, const TCHAR* InMarker, const int32 InMarkerLen);
bool PeekInsensitiveMarker(const TCHAR* Buffer, const TCHAR* InMarker, const int32 InMarkerLen);

#define TEXT_STRINGIFICATION_SKIP_MARKER(T)					TEXT_STRINGIFICATION_FUNC_MODIFY_BUFFER_AND_VALIDATE(TextStringificationUtil::SkipMarker, T, UE_ARRAY_COUNT(T) - 1)
#define TEXT_STRINGIFICATION_SKIP_INSENSITIVE_MARKER(T)		TEXT_STRINGIFICATION_FUNC_MODIFY_BUFFER_AND_VALIDATE(TextStringificationUtil::SkipInsensitiveMarker, T, UE_ARRAY_COUNT(T) - 1)
#define TEXT_STRINGIFICATION_SKIP_MARKER_LEN(T)				Buffer += (UE_ARRAY_COUNT(T) - 1)
const TCHAR* SkipMarker(const TCHAR* Buffer, const TCHAR* InMarker, const int32 InMarkerLen);
const TCHAR* SkipInsensitiveMarker(const TCHAR* Buffer, const TCHAR* InMarker, const int32 InMarkerLen);

#define TEXT_STRINGIFICATION_SKIP_WHITESPACE()				TEXT_STRINGIFICATION_FUNC_MODIFY_BUFFER_AND_VALIDATE(TextStringificationUtil::SkipWhitespace)
const TCHAR* SkipWhitespace(const TCHAR* Buffer);

#define TEXT_STRINGIFICATION_SKIP_WHITESPACE_TO_CHAR(C)		TEXT_STRINGIFICATION_FUNC_MODIFY_BUFFER_AND_VALIDATE(TextStringificationUtil::SkipWhitespaceToCharacter, TEXT(C))
const TCHAR* SkipWhitespaceToCharacter(const TCHAR* Buffer, const TCHAR InChar);

#define TEXT_STRINGIFICATION_SKIP_WHITESPACE_AND_CHAR(C)	TEXT_STRINGIFICATION_FUNC_MODIFY_BUFFER_AND_VALIDATE(TextStringificationUtil::SkipWhitespaceAndCharacter, TEXT(C))
const TCHAR* SkipWhitespaceAndCharacter(const TCHAR* Buffer, const TCHAR InChar);

#define TEXT_STRINGIFICATION_READ_NUMBER(V)					TEXT_STRINGIFICATION_FUNC_MODIFY_BUFFER_AND_VALIDATE(TextStringificationUtil::ReadNumberFromBuffer, V)
const TCHAR* ReadNumberFromBuffer(const TCHAR* Buffer, FFormatArgumentValue& OutValue);

#define TEXT_STRINGIFICATION_READ_ALNUM(V)					TEXT_STRINGIFICATION_FUNC_MODIFY_BUFFER_AND_VALIDATE(TextStringificationUtil::ReadAlnumFromBuffer, V)
const TCHAR* ReadAlnumFromBuffer(const TCHAR* Buffer, FString& OutValue);

#define TEXT_STRINGIFICATION_READ_QUOTED_STRING(V)			TEXT_STRINGIFICATION_FUNC_MODIFY_BUFFER_AND_VALIDATE(TextStringificationUtil::ReadQuotedStringFromBuffer, V)
const TCHAR* ReadQuotedStringFromBuffer(const TCHAR* Buffer, FString& OutStr);

#define TEXT_STRINGIFICATION_READ_SCOPED_ENUM(S, V)			TEXT_STRINGIFICATION_FUNC_MODIFY_BUFFER_AND_VALIDATE(TextStringificationUtil::ReadScopedEnumFromBuffer, S, V)
template <typename T>
const TCHAR* ReadScopedEnumFromBuffer(const TCHAR* Buffer, const FString& Scope, T& OutValue)
{
	if (PeekInsensitiveMarker(Buffer, *Scope, Scope.Len()))
	{
		// Parsing something of the form: EEnumName::...
		Buffer += Scope.Len();

		FString EnumValueString;
		Buffer = ReadAlnumFromBuffer(Buffer, EnumValueString);

		if (Buffer && LexTryParseString(OutValue, *EnumValueString))
		{
			return Buffer;
		}
	}

	return nullptr;
}

template <typename T>
void WriteScopedEnumToBuffer(FString& Buffer, const TCHAR* Scope, const T Value)
{
	Buffer += Scope;
	Buffer += LexToString(Value);
}

}	// namespace TextStringificationUtil

/** Base interface class for all FText history types */
class FTextHistory : public ITextData, public TRefCountingMixin<FTextHistory>
{
public:
	FTextHistory() = default;
	virtual ~FTextHistory() = default;

	/** Disallow copying */
	FTextHistory(const FTextHistory&) = delete;
	FTextHistory& operator=(FTextHistory&) = delete;

	//~ IRefCountedObject
	virtual void AddRef() const override final { TRefCountingMixin<FTextHistory>::AddRef(); }
	virtual FReturnedRefCountValue Release() const override final { return TRefCountingMixin<FTextHistory>::Release(); }
	virtual FReturnedRefCountValue GetRefCount() const override final { return TRefCountingMixin<FTextHistory>::GetRefCount(); }

	//~ ITextData
	virtual const FString& GetSourceString() const override { return GetDisplayString(); }
	virtual FTextConstDisplayStringPtr GetLocalizedString() const override { return nullptr; }
	virtual uint16 GetGlobalHistoryRevision() const override final;
	virtual uint16 GetLocalHistoryRevision() const override final;
	virtual const FTextHistory& GetTextHistory() const override final { return *this; }
	virtual FTextHistory& GetMutableTextHistory() override final { return *this; }

	/** Get the type of this history */
	virtual ETextHistoryType GetType() const = 0;

	/**
	 * Returns the ID of the shared display string (if any).
	 */
	virtual FTextId GetTextId() const { return FTextId(); }

	/**
	 * Build the display string for the invariant culture
	 */
	virtual FString BuildInvariantDisplayString() const = 0;

	/**
	 * Check whether this history is considered identical to the other history, based on the comparison flags provided.
	 * @note You must ensure that both histories are the same type (via GetType) prior to calling this function!
	 */
	virtual bool IdenticalTo(const FTextHistory& Other, const ETextIdenticalModeFlags CompareModeFlags) const = 0;

	/** Serializes the history to/from a structured archive slot */
	virtual void Serialize(FStructuredArchive::FRecord Record) = 0;

	/**
	 * Check the given stream of text to see if it looks like something this class could process in via ReadFromBuffer.
	 * @note This doesn't guarantee that ReadFromBuffer will be able to process the stream, only that it could attempt to.
	 */
	static  bool StaticShouldReadFromBuffer(const TCHAR* Buffer) { return false; }
	virtual bool ShouldReadFromBuffer(const TCHAR* Buffer) const { return StaticShouldReadFromBuffer(Buffer); }

	/**
	 * Attempt to parse this text history from the given stream of text.
	 *
	 * @param Buffer			The buffer of text to read from.
	 * @param TextNamespace		An optional namespace to use when parsing texts that use LOCTEXT (default is an empty namespace).
	 * @param PackageNamespace	The package namespace of the containing object (if loading for a property - see TextNamespaceUtil::GetPackageNamespace).
	 * @param OutDisplayString	The display string pointer to potentially fill in (depending on the history type).
	 *
	 * @return The updated buffer after we parsed this text history, or nullptr on failure
	 */
	virtual const TCHAR* ReadFromBuffer(const TCHAR* Buffer, const TCHAR* TextNamespace, const TCHAR* PackageNamespace) { return nullptr; }

	/**
	 * Write this text history to a stream of text
	 *
	 * @param Buffer				 The buffer of text to write to.
	 * @param DisplayString			 The display string associated with the text being written
	 * @param bStripPackageNamespace True to strip the package namespace from the written NSLOCTEXT value (eg, when saving cooked data)
	 *
	 * @return True if we wrote valid data into Buffer, false otherwise
	 */
	virtual bool WriteToBuffer(FString& Buffer, const bool bStripPackageNamespace) const { return false; }

	/** Get any historic text format data from this history */
	virtual void GetHistoricFormatData(const FText& InText, TArray<FHistoricTextFormatData>& OutHistoricFormatData) const {}

	/** Get any historic numeric format data from this history */
	virtual bool GetHistoricNumericData(const FText& InText, FHistoricTextNumericData& OutHistoricNumericData) const { return false; }
	
#if WITH_EDITORONLY_DATA
	virtual const FString& GetDevNotes() const;
#endif

	/** Update the display string if the history is out-of-date */
	void UpdateDisplayStringIfOutOfDate();

protected:
	/** True if "UpdateDisplayString" might do something if called, or False if it would be redundant */
	virtual bool CanUpdateDisplayString() { return true; }

	/** Update the display string when the history is out-of-date */
	virtual void UpdateDisplayString() = 0;

	/** Mark the history revisions as out-of-date */
	void MarkDisplayStringOutOfDate();

	/** Mark the history revisions as up-to-date */
	void MarkDisplayStringUpToDate();

private:
	/** Global revision index of this history, rebuilds when it is out of sync with the FTextLocalizationManager */
	uint16 GlobalRevision = 0;
	/** Local revision index of this history, rebuilds when it is out of sync with the FTextLocalizationManager */
	uint16 LocalRevision = 0;
	/** Mutex to prevent races when multiple threads are updating out of date state. */
	mutable UE::FTransactionallySafeMutex Mutex = UE::FTransactionallySafeMutex();
};

/** A potentially localized piece of source text (may have a TextId). */
class FTextHistory_Base : public FTextHistory
{
public:
	FTextHistory_Base() = default;
	FTextHistory_Base(const FTextId& InTextId, FString&& InSourceString);
	FTextHistory_Base(const FTextId& InTextId, FString&& InSourceString, FTextConstDisplayStringPtr&& InLocalizedString);
#if WITH_EDITORONLY_DATA
	FTextHistory_Base(const FTextId& InTextId, FString&& InSourceString, FString&& InDevNotes);
#endif

	//~ Begin FTextHistory Interface
	OVERRIDE_TEXT_HISTORY_STRINGIFICATION;
	virtual ETextHistoryType GetType() const override { return ETextHistoryType::Base; }
	virtual FTextId GetTextId() const override final;
	virtual FTextConstDisplayStringPtr GetLocalizedString() const override;
	virtual const FString& GetSourceString() const override;
	virtual const FString& GetDisplayString() const override;
	virtual FString BuildInvariantDisplayString() const override;
	virtual bool IdenticalTo(const FTextHistory& Other, const ETextIdenticalModeFlags CompareModeFlags) const override;
	virtual void Serialize(FStructuredArchive::FRecord Record) override;
	virtual bool CanUpdateDisplayString() override;
	virtual void UpdateDisplayString() override;
#if WITH_EDITORONLY_DATA
	virtual const FString& GetDevNotes() const override;
#endif
	//~ End FTextHistory Interface

private:
	/** The ID for an FText (if any) */
	FTextId TextId;
	/** The source string for an FText */
	FString SourceString;
	/** The localized string (from FTextLocalizationManager, if any) */
	FTextConstDisplayStringPtr LocalizedString;
	
#if WITH_EDITORONLY_DATA
	/** The developer notes to give translators context on this string */
	FString DevNotes;
#endif
};

/** Base class for text histories that hold a generated display string. */
class FTextHistory_Generated : public FTextHistory
{
public:
	FTextHistory_Generated() = default;
	explicit FTextHistory_Generated(FString&& InDisplayString);

	//~ Begin FTextHistory Interface
	virtual FTextId GetTextId() const override final { return FTextId(); }
	virtual const FString& GetDisplayString() const override;
	virtual void Serialize(FStructuredArchive::FRecord Record) override;
	virtual void UpdateDisplayString() override;
	//~ End FTextHistory Interface

protected:
	/**
	 * Build the display string for the current culture
	 */
	virtual FString BuildLocalizedDisplayString() const = 0;

	/** The generated display string */
	FString DisplayString;
};

/** Handles history for FText::Format when passing named arguments */
class FTextHistory_NamedFormat : public FTextHistory_Generated
{
public:
	FTextHistory_NamedFormat() = default;
	FTextHistory_NamedFormat(FString&& InDisplayString, FTextFormat&& InSourceFmt, FFormatNamedArguments&& InArguments);

	//~ Begin FTextHistory Interface
	OVERRIDE_TEXT_HISTORY_STRINGIFICATION;
	virtual ETextHistoryType GetType() const override { return ETextHistoryType::NamedFormat; }
	virtual bool IdenticalTo(const FTextHistory& Other, const ETextIdenticalModeFlags CompareModeFlags) const override;
	virtual FString BuildLocalizedDisplayString() const override;
	virtual FString BuildInvariantDisplayString() const override;
	virtual void Serialize(FStructuredArchive::FRecord Record) override;
	virtual void GetHistoricFormatData(const FText& InText, TArray<FHistoricTextFormatData>& OutHistoricFormatData) const override;
	//~ End FTextHistory Interface

private:
	/** The pattern used to format the text */
	FTextFormat SourceFmt;
	/** Arguments to replace in the pattern string */
	FFormatNamedArguments Arguments;
};

/** Handles history for FText::Format when passing ordered arguments */
class FTextHistory_OrderedFormat : public FTextHistory_Generated
{
public:
	FTextHistory_OrderedFormat() = default;
	FTextHistory_OrderedFormat(FString&& InDisplayString, FTextFormat&& InSourceFmt, FFormatOrderedArguments&& InArguments);

	//~ Begin FTextHistory Interface
	OVERRIDE_TEXT_HISTORY_STRINGIFICATION;
	virtual ETextHistoryType GetType() const override { return ETextHistoryType::OrderedFormat; }
	virtual bool IdenticalTo(const FTextHistory& Other, const ETextIdenticalModeFlags CompareModeFlags) const override;
	virtual FString BuildLocalizedDisplayString() const override;
	virtual FString BuildInvariantDisplayString() const override;
	virtual void Serialize(FStructuredArchive::FRecord Record) override;
	virtual void GetHistoricFormatData(const FText& InText, TArray<FHistoricTextFormatData>& OutHistoricFormatData) const override;
	//~ End FTextHistory Interface

private:
	/** The pattern used to format the text */
	FTextFormat SourceFmt;
	/** Arguments to replace in the pattern string */
	FFormatOrderedArguments Arguments;
};

/** Handles history for FText::Format when passing raw argument data */
class FTextHistory_ArgumentDataFormat : public FTextHistory_Generated
{
public:
	FTextHistory_ArgumentDataFormat() = default;
	FTextHistory_ArgumentDataFormat(FString&& InDisplayString, FTextFormat&& InSourceFmt, TArray<FFormatArgumentData>&& InArguments);

	//~ Begin FTextHistory Interface
	OVERRIDE_TEXT_HISTORY_STRINGIFICATION;
	virtual ETextHistoryType GetType() const override { return ETextHistoryType::ArgumentFormat; }
	virtual bool IdenticalTo(const FTextHistory& Other, const ETextIdenticalModeFlags CompareModeFlags) const override;
	virtual FString BuildLocalizedDisplayString() const override;
	virtual FString BuildInvariantDisplayString() const override;
	virtual void Serialize(FStructuredArchive::FRecord Record) override;
	virtual void GetHistoricFormatData(const FText& InText, TArray<FHistoricTextFormatData>& OutHistoricFormatData) const override;
	//~ End FTextHistory Interface

private:
	/** The pattern used to format the text */
	FTextFormat SourceFmt;
	/** Arguments to replace in the pattern string */
	TArray<FFormatArgumentData> Arguments;
};

/** Base class for managing formatting FText's from: AsNumber, AsPercent, and AsCurrency. Manages data serialization of these history events */
class FTextHistory_FormatNumber : public FTextHistory_Generated
{
public:
	FTextHistory_FormatNumber() = default;
	FTextHistory_FormatNumber(FString&& InDisplayString, FFormatArgumentValue InSourceValue, const FNumberFormattingOptions* const InFormatOptions, FCulturePtr InTargetCulture);

	//~ Begin FTextHistory Interface
	virtual bool IdenticalTo(const FTextHistory& Other, const ETextIdenticalModeFlags CompareModeFlags) const override;
	virtual void Serialize(FStructuredArchive::FRecord Record) override;
	//~ End FTextHistory interface

protected:
	/** Build the numeric display string using the given formatting rules */
	FString BuildNumericDisplayString(const FDecimalNumberFormattingRules& InFormattingRules, const int32 InValueMultiplier = 1) const;

	/** The source value to format from */
	FFormatArgumentValue SourceValue;
	/** All the formatting options available to format using. This can be empty. */
	TOptional<FNumberFormattingOptions> FormatOptions;
	/** The culture to format using */
	FCulturePtr TargetCulture;
};

/**  Handles history for formatting using AsNumber */
class FTextHistory_AsNumber : public FTextHistory_FormatNumber
{
public:
	FTextHistory_AsNumber() = default;
	FTextHistory_AsNumber(FString&& InDisplayString, FFormatArgumentValue InSourceValue, const FNumberFormattingOptions* const InFormatOptions, FCulturePtr InTargetCulture);

	//~ Begin FTextHistory Interface
	OVERRIDE_TEXT_HISTORY_STRINGIFICATION;
	virtual ETextHistoryType GetType() const override { return ETextHistoryType::AsNumber; }
	virtual FString BuildLocalizedDisplayString() const override;
	virtual FString BuildInvariantDisplayString() const override;
	virtual void Serialize(FStructuredArchive::FRecord Record) override;
	virtual bool GetHistoricNumericData(const FText& InText, FHistoricTextNumericData& OutHistoricNumericData) const override;
	//~ End FTextHistory interface
};

/**  Handles history for formatting using AsPercent */
class FTextHistory_AsPercent : public FTextHistory_FormatNumber
{
public:
	FTextHistory_AsPercent() = default;
	FTextHistory_AsPercent(FString&& InDisplayString, FFormatArgumentValue InSourceValue, const FNumberFormattingOptions* const InFormatOptions, FCulturePtr InTargetCulture);

	//~ Begin FTextHistory Interface
	OVERRIDE_TEXT_HISTORY_STRINGIFICATION;
	virtual ETextHistoryType GetType() const override { return ETextHistoryType::AsPercent; }
	virtual FString BuildLocalizedDisplayString() const override;
	virtual FString BuildInvariantDisplayString() const override;
	virtual void Serialize(FStructuredArchive::FRecord Record) override;
	virtual bool GetHistoricNumericData(const FText& InText, FHistoricTextNumericData& OutHistoricNumericData) const override;
	//~ End FTextHistory interface
};

/**  Handles history for formatting using AsCurrency */
class FTextHistory_AsCurrency : public FTextHistory_FormatNumber
{
public:
	FTextHistory_AsCurrency() = default;
	FTextHistory_AsCurrency(FString&& InDisplayString, FFormatArgumentValue InSourceValue, FString InCurrencyCode, const FNumberFormattingOptions* const InFormatOptions, FCulturePtr InTargetCulture);

	//~ Begin FTextHistory Interface
	OVERRIDE_TEXT_HISTORY_STRINGIFICATION;
	virtual ETextHistoryType GetType() const override { return ETextHistoryType::AsCurrency; }
	virtual FString BuildLocalizedDisplayString() const override;
	virtual FString BuildInvariantDisplayString() const override;
	virtual void Serialize(FStructuredArchive::FRecord Record) override;
	//~ End FTextHistory Interface

private:
	/** The currency used to format the number. */
	FString CurrencyCode;
};

/**  Handles history for formatting using AsDate */
class FTextHistory_AsDate : public FTextHistory_Generated
{
public:
	FTextHistory_AsDate() = default;
	FTextHistory_AsDate(FString&& InDisplayString, FDateTime InSourceDateTime, const EDateTimeStyle::Type InDateStyle, FString InTimeZone, FCulturePtr InTargetCulture);

	//~ Begin FTextHistory Interface
	OVERRIDE_TEXT_HISTORY_STRINGIFICATION;
	virtual ETextHistoryType GetType() const override { return ETextHistoryType::AsDate; }
	virtual bool IdenticalTo(const FTextHistory& Other, const ETextIdenticalModeFlags CompareModeFlags) const override;
	virtual FString BuildLocalizedDisplayString() const override;
	virtual FString BuildInvariantDisplayString() const override;
	virtual void Serialize(FStructuredArchive::FRecord Record) override;
	//~ End FTextHistory Interface

private:
	/** The source date structure to format */
	FDateTime SourceDateTime;
	/** Style to format the date using */
	EDateTimeStyle::Type DateStyle;
	/** Timezone to put the time in */
	FString TimeZone;
	/** Culture to format the date in */
	FCulturePtr TargetCulture;
};

/**  Handles history for formatting using AsTime */
class FTextHistory_AsTime : public FTextHistory_Generated
{
public:
	FTextHistory_AsTime() = default;
	FTextHistory_AsTime(FString&& InDisplayString, FDateTime InSourceDateTime, const EDateTimeStyle::Type InTimeStyle, FString InTimeZone, FCulturePtr InTargetCulture);

	//~ Begin FTextHistory Interface
	OVERRIDE_TEXT_HISTORY_STRINGIFICATION;
	virtual ETextHistoryType GetType() const override { return ETextHistoryType::AsTime; }
	virtual bool IdenticalTo(const FTextHistory& Other, const ETextIdenticalModeFlags CompareModeFlags) const override;
	virtual FString BuildLocalizedDisplayString() const override;
	virtual FString BuildInvariantDisplayString() const override;
	virtual void Serialize(FStructuredArchive::FRecord Record) override;
	//~ End FTextHistory Interface

private:
	/** The source time structure to format */
	FDateTime SourceDateTime;
	/** Style to format the time using */
	EDateTimeStyle::Type TimeStyle;
	/** Timezone to put the time in */
	FString TimeZone;
	/** Culture to format the time in */
	FCulturePtr TargetCulture;
};

/**  Handles history for formatting using AsDateTime */
class FTextHistory_AsDateTime : public FTextHistory_Generated
{
public:
	FTextHistory_AsDateTime() = default;
	FTextHistory_AsDateTime(FString&& InDisplayString, FDateTime InSourceDateTime, const EDateTimeStyle::Type InDateStyle, const EDateTimeStyle::Type InTimeStyle, FString InTimeZone, FCulturePtr InTargetCulture);
	FTextHistory_AsDateTime(FString&& InDisplayString, FDateTime InSourceDateTime, FString InCustomPattern, FString InTimeZone, FCulturePtr InTargetCulture);

	//~ Begin FTextHistory Interface
	OVERRIDE_TEXT_HISTORY_STRINGIFICATION;
	virtual ETextHistoryType GetType() const override { return ETextHistoryType::AsDateTime; }
	virtual bool IdenticalTo(const FTextHistory& Other, const ETextIdenticalModeFlags CompareModeFlags) const override;
	virtual FString BuildLocalizedDisplayString() const override;
	virtual FString BuildInvariantDisplayString() const override;
	virtual void Serialize(FStructuredArchive::FRecord Record) override;
	//~ End FTextHistory Interfaces

private:
	/** The source date and time structure to format */
	FDateTime SourceDateTime;
	/** Style to format the date using */
	EDateTimeStyle::Type DateStyle;
	/** Style to format the time using */
	EDateTimeStyle::Type TimeStyle;
	/** Custom pattern for this format (if DateStyle == Custom) */
	FString CustomPattern;
	/** Timezone to put the time in */
	FString TimeZone;
	/** Culture to format the time in */
	FCulturePtr TargetCulture;
};

/**  Handles history for transforming text (eg, ToLower/ToUpper) */
class FTextHistory_Transform : public FTextHistory_Generated
{
public:
	enum class ETransformType : uint8
	{
		ToLower = 0,
		ToUpper,

		// Add new enum types at the end only! They are serialized by index.
	};

	FTextHistory_Transform() = default;
	FTextHistory_Transform(FString&& InDisplayString, FText InSourceText, const ETransformType InTransformType);

	//~ Begin FTextHistory Interface
	OVERRIDE_TEXT_HISTORY_STRINGIFICATION;
	virtual ETextHistoryType GetType() const override { return ETextHistoryType::Transform; }
	virtual bool IdenticalTo(const FTextHistory& Other, const ETextIdenticalModeFlags CompareModeFlags) const override;
	virtual FString BuildLocalizedDisplayString() const override;
	virtual FString BuildInvariantDisplayString() const override;
	virtual void Serialize(FStructuredArchive::FRecord Record) override;
	virtual void GetHistoricFormatData(const FText& InText, TArray<FHistoricTextFormatData>& OutHistoricFormatData) const override;
	virtual bool GetHistoricNumericData(const FText& InText, FHistoricTextNumericData& OutHistoricNumericData) const override;
#if WITH_EDITORONLY_DATA
	virtual const FString& GetDevNotes() const override;
#endif
	//~ End FTextHistory Interfaces

private:
	/** The source text instance that was transformed */
	FText SourceText;
	/** How the source text was transformed */
	ETransformType TransformType;
};

/** Holds a pointer to a referenced display string from a string table. */
class FTextHistory_StringTableEntry : public FTextHistory
{
public:
	FTextHistory_StringTableEntry() = default;
	FTextHistory_StringTableEntry(FName InTableId, const FTextKey& InKey, const EStringTableLoadingPolicy InLoadingPolicy);

	//~ Begin FTextHistory Interface
	OVERRIDE_TEXT_HISTORY_STRINGIFICATION;
	virtual ETextHistoryType GetType() const override { return ETextHistoryType::StringTableEntry; }
	virtual FTextId GetTextId() const override final;
	virtual FTextConstDisplayStringPtr GetLocalizedString() const override;
	virtual const FString& GetSourceString() const override;
	virtual const FString& GetDisplayString() const override;
	virtual FString BuildInvariantDisplayString() const override;
	virtual bool IdenticalTo(const FTextHistory& Other, const ETextIdenticalModeFlags CompareModeFlags) const override;
	virtual void Serialize(FStructuredArchive::FRecord Record) override;
	virtual void UpdateDisplayString() override;
#if WITH_EDITORONLY_DATA
	virtual const FString& GetDevNotes() const override;
#endif
	//~ End FTextHistory Interface

	void GetTableIdAndKey(FName& OutTableId, FTextKey& OutKey) const;

private:
	enum class EStringTableLoadingPhase : uint8
	{
		/** This string table is pending load, and load should be attempted when possible */
		PendingLoad,
		/** This string table is currently being loaded, potentially asynchronously */
		Loading,
		/** This string was loaded, though that load may have failed */
		Loaded,
	};

	/** Hosts the reference data for this text history */
	class FStringTableReferenceData : public TSharedFromThis<FStringTableReferenceData, ESPMode::ThreadSafe>
	{
	public:
		/** Initialize this data, immediately starting an asset load if required and possible */
		void Initialize(FName InTableId, const FTextKey& InKey, const EStringTableLoadingPolicy InLoadingPolicy);

		/** Check whether this instance is considered identical to the other instance */
		bool IsIdentical(const FStringTableReferenceData& Other) const;

		/** Get the string table ID being referenced */
		FName GetTableId() const;

		/** Get the key within the string table being referenced */
		FTextKey GetKey() const;

		/** Get the table ID and key within it that are being referenced */
		void GetTableIdAndKey(FName& OutTableId, FTextKey& OutKey) const;

		/** Collect any string table asset references */
		void CollectStringTableAssetReferences(FStructuredArchive::FRecord Record);

		/** Get the localized ID of this string table (if any). */
		FTextId GetTextId();

		/** Resolve the string table pointer, potentially re-caching it if it's missing or stale */
		FStringTableEntryConstPtr ResolveStringTableEntry();

		/** Resolve the display string pointer, potentially re-caching it if the string table entry is missing or stale */
		FTextConstDisplayStringPtr ResolveDisplayString(const bool bForceRefresh = false);

	private:
		/** Begin an asset load if required and possible */
		void ConditionalBeginAssetLoad();

		/** The string table ID being referenced */
		FName TableId;

		/** The key within the string table being referenced */
		FTextKey Key;

		/** The loading phase of any referenced string table asset */
		EStringTableLoadingPhase LoadingPhase = EStringTableLoadingPhase::PendingLoad;

		/** Cached string table entry pointer */
		FStringTableEntryConstWeakPtr StringTableEntry;

		/** Cached display string pointer */
		FTextConstDisplayStringPtr DisplayString;

		/** Critical section preventing concurrent access to the resolved data */
		mutable FTransactionallySafeCriticalSection DataCS;
	};
	typedef TSharedPtr<FStringTableReferenceData, ESPMode::ThreadSafe> FStringTableReferenceDataPtr;
	typedef TWeakPtr<FStringTableReferenceData, ESPMode::ThreadSafe> FStringTableReferenceDataWeakPtr;

	/** The reference data for this text history */
	FStringTableReferenceDataPtr StringTableReferenceData;
};

/** Handles history for FText::FromTextGenerator */
class FTextHistory_TextGenerator : public FTextHistory_Generated
{
public:
	FTextHistory_TextGenerator() = default;
	FTextHistory_TextGenerator(FString&& InDisplayString, const TSharedRef<ITextGenerator>& InTextGenerator);

	//~ Begin FTextHistory Interface
	virtual ETextHistoryType GetType() const override { return ETextHistoryType::TextGenerator; }
	virtual bool IdenticalTo(const FTextHistory& Other, const ETextIdenticalModeFlags CompareModeFlags) const override;
	virtual FString BuildLocalizedDisplayString() const override;
	virtual FString BuildInvariantDisplayString() const override;
	virtual void Serialize(FStructuredArchive::FRecord Record) override;
	//~ End FTextHistory Interface

private:
	/** The object implementing the custom generation code */
	TSharedPtr<ITextGenerator> TextGenerator;
};

#undef OVERRIDE_TEXT_HISTORY_STRINGIFICATION
