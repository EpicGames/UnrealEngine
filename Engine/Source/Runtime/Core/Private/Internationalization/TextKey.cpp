// Copyright Epic Games, Inc. All Rights Reserved.

#include "Internationalization/TextKey.h"

#include "AutoRTFM.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/ChunkedArray.h"
#include "Containers/Map.h"
#include "CoreGlobals.h"
#include "HAL/LowLevelMemTracker.h"
#include "Hash/CityHash.h"
#include "Logging/LogMacros.h"
#include "Misc/Guid.h"
#include "Misc/LazySingleton.h"
#include "Misc/ScopeRWLock.h"
#include "Misc/TransactionallySafeRWLock.h"
#include "Modules/VisualizerDebuggingState.h"
#include "Memory/LinearAllocator.h"

DEFINE_LOG_CATEGORY_STATIC(LogTextKey, Log, All);

// Note: If setting this to 0, you'll also want to update the FTextKey natvis to change ",s8" to ",su"
#ifndef UE_TEXTKEY_USE_UTF8
	#define UE_TEXTKEY_USE_UTF8 (1)
#endif

#ifndef UE_TEXTKEY_SPLIT_GUID
	#define UE_TEXTKEY_SPLIT_GUID (1)
#endif

#ifndef UE_TEXTKEY_ELEMENTS_MIN_HASH_SIZE
	#define UE_TEXTKEY_ELEMENTS_MIN_HASH_SIZE (32768)
#endif

#ifndef UE_TEXTKEY_USE_INLINE_ALLOCATOR
	#define UE_TEXTKEY_USE_INLINE_ALLOCATOR (0)
#endif

#ifndef UE_TEXTKEY_MAX_SIZE_BYTES
	#define UE_TEXTKEY_MAX_SIZE_BYTES (1000)
#endif

#if UE_TEXTKEY_USE_UTF8
using FTextKeyCharType = UTF8CHAR;
#else
using FTextKeyCharType = TCHAR;
#endif
using FTextKeyStringView = TStringView<FTextKeyCharType>;


// FName-like block-based variable sized allocator
template<typename InElementType, uint32 ChunkCursorBits = 16> // Default to 64K per chunk
class TChunkedPackedArray
{
public:
	static constexpr uint32 BytesPerChunk = (1 << ChunkCursorBits);

	int32 Add(const uint32 ElementSize)
	{
		if (Chunks.Num() == 0 || ChunkCursors[Chunks.Num() - 1] + ElementSize > BytesPerChunk)
		{
			AllocateNewChunk();
		}

		const uint32 CurrentChunk = Chunks.Num() - 1; 
		int32 ResultIndex = (CurrentChunk << ChunkCursorBits) | ChunkCursors[CurrentChunk];
		ChunkCursors[CurrentChunk] += ElementSize;
		Count++;
		
		return ResultIndex;
	}

	int32 GetFirstIndex() const
	{
		return Num() > 0 ? 0 : INDEX_NONE;
	}

	int32 GetNextIndex(const int32 ElementIndex, const uint32 ElementSize) const
	{
		uint32 ChunkIndex = ElementIndex >> ChunkCursorBits;
		uint32 ChunkCursor = ElementIndex & ((1 << ChunkCursorBits) - 1);
		
		ChunkCursor += ElementSize;

		if (ChunkCursor >= ChunkCursors[ChunkIndex])
		{
			ChunkIndex++;
			ChunkCursor = 0;
		}
		return (ChunkIndex << ChunkCursorBits) | ChunkCursor;
	}

	InElementType& Get(const int32 ElementIndex) const
	{
		uint32 ChunkIndex = ElementIndex >> ChunkCursorBits;
		uint32 ChunkCursor = ElementIndex & ((1 << ChunkCursorBits) - 1);

		return *reinterpret_cast<InElementType*>(&Chunks[ChunkIndex][ChunkCursor]);
	}

	uint32 Num() const
	{
		return Count;
	}
	
	~TChunkedPackedArray()
	{
		for(uint8* Chunk : Chunks)
		{
			FMemory::Free(Chunk);
		}
	}
	
private:
	TArray<uint8*> Chunks;
	TArray<uint32> ChunkCursors;
	uint32 Count = 0;

	void AllocateNewChunk()
	{
		Chunks.Add(static_cast<uint8*>(FMemory::Malloc(BytesPerChunk, 8)));
		ChunkCursors.Add(0);
	}
};

class FTextKeyState
{
public:
	FTextKeyState()
	{
		// Register the natvis data accessor
		UE::Core::EVisualizerDebuggingStateResult Result = UE::Core::FVisualizerDebuggingState::Assign(FGuid(0xd31281c0, 0x182b4419, 0x814e25be, 0x4b7e7b41), this);
	}

	void FindOrAdd(FStringView InStr, FTextKey& OutTextKey)
	{
		check(!InStr.IsEmpty());

		// Note: This hash gets serialized so *DO NOT* change it without fixing the serialization to discard the old hash method (also update FTextKey::GetTypeHash)
		const uint32 StrHash = TextKeyUtil::HashString(InStr);

		// Open around adding this in a cache, if we abort just leak the value in the cache
		// as the cache takes ownership
		UE_AUTORTFM_OPEN
		{
			FindOrAddImpl(InStr, StrHash, OutTextKey);
		};
	}

	void FindOrAdd(FStringView InStr, const uint32 InStrHash, FTextKey& OutTextKey)
	{
		check(!InStr.IsEmpty());

		// Open around adding this in a cache, if we abort just leak the value in the cache
		// as the cache takes ownership
		UE_AUTORTFM_OPEN
		{
			FindOrAddImpl(InStr, InStrHash, OutTextKey);
		};
	}

	void AppendStringByIndex(const int32 InIndex, FString& Out) const
	{
		check(InIndex != INDEX_NONE);

		UE::TReadScopeLock ScopeLock(DataRW);

		const FKeyData& KeyData = KeyDataAllocations.Get(InIndex);
		if (KeyData.IsStringView())
		{
			Out += KeyData.AsStringView();
		}
		else
		{
			KeyData.AsGuid().AppendString(Out, EGuidFormats::Digits);
		}
	}

	void AppendStringByIndex(const int32 InIndex, FStringBuilderBase& Out) const
	{
		check(InIndex != INDEX_NONE);

		UE::TReadScopeLock ScopeLock(DataRW);

		const FKeyData& KeyData = KeyDataAllocations.Get(InIndex);
		if (KeyData.IsStringView())
		{
			Out += KeyData.AsStringView();
		}
		else
		{
			KeyData.AsGuid().AppendString(Out, EGuidFormats::Digits);
		}
	}

	uint32 GetHashByIndex(const int32 InIndex) const
	{
		check(InIndex != INDEX_NONE);

		UE::TReadScopeLock ScopeLock(DataRW);

		const FKeyData& KeyData = KeyDataAllocations.Get(InIndex);
		return KeyData.StrHash;
	}

	void Shrink()
	{
		// Note: Nothing to shrink as things grow in chunks
	}

	static FTextKeyState& GetState()
	{
		return TLazySingleton<FTextKeyState>::Get();
	}

	static void TearDown()
	{
		return TLazySingleton<FTextKeyState>::TearDown();
	}

private:
	// Used to represent Key Data to find or insert into the allocator
	struct FKeyDataView
	{
		FKeyDataView(FTextKeyStringView InStr, const uint32 InStrHash)
			: StrLen(InStr.Len())
			, StrHash(InStrHash)
			, DataPtr(InStr.GetData())
		{
		}

		FKeyDataView(const FGuid& InGuid, const uint32 InStrHash)
			: StrLen(INDEX_NONE)
			, StrHash(InStrHash)
			, DataPtr(&InGuid)
		{
		}
		
		bool IsStringView() const
		{
			return StrLen >= 0;
		}

		bool IsGuid() const
		{
			return StrLen == INDEX_NONE;
		}

		FTextKeyStringView AsStringView() const
		{
			checkf(IsStringView(), TEXT("AsStringView called on a FKeyDataView that doesn't reference a string!"));
			return FTextKeyStringView(static_cast<const FTextKeyCharType*>(DataPtr), StrLen);
		}

		const FGuid& AsGuid() const
		{
			checkf(IsGuid(), TEXT("AsGuid called on a FKeyDataView that doesn't reference a GUID!"));
			return *static_cast<const FGuid*>(DataPtr);
		}

		/** Length of the string (in characters) if DataPtr points to a string buffer, or INDEX_NONE if it points to a FGuid */
		int32 StrLen;
		/** Hash of the source string this data was created from */
		uint32 StrHash;
		/** Internal data pointer; if StrLen >= 0 this points to a string buffer (which may not be null-terminated), or if INDEX_NONE points to a FGuid */
		const void* DataPtr;
	};

	// Used to represent Key Data that exists inside the allocator
	struct FKeyData
	{
		bool IsStringView() const
		{
			return StrLen >= 0;
		}

		FTextKeyStringView AsStringView() const
		{
			checkf(IsStringView(), TEXT("AsStringView called on a FKeyData that doesn't reference a string!"));
			return FTextKeyStringView(reinterpret_cast<const FTextKeyCharType*>(DataPtr), StrLen);
		}

		bool IsGuid() const
		{
			return StrLen == INDEX_NONE;
		}

		const FGuid& AsGuid() const
		{
			checkf(IsGuid(), TEXT("AsGuid called on a FKeyData that doesn't reference a GUID!"));
			return *reinterpret_cast<const FGuid*>(DataPtr);
		}

		/** Index of the next element in this hash bucket */
		int32 NextElementIndex = INDEX_NONE;
		/** Hash of the source string this data was created from */
		uint32 StrHash;
		
#if UE_TEXTKEY_USE_INLINE_ALLOCATOR
		/** Length of the string (in characters) if DataPtr points to a string buffer, or INDEX_NONE if it points to a FGuid */
		int16 StrLen;

		/** Internal data pointer; if StrLen >= 0 this points to a string buffer (which may not be null-terminated), or if INDEX_NONE points to a FGuid */
		uint8 DataPtr[0];
#else
		/** Length of the string (in characters) if DataPtr points to a string buffer, or INDEX_NONE if it points to a FGuid */
		int32 StrLen;

		const void* DataPtr;
#endif
	};

	static constexpr uint32 KeyDataHeaderSizeBytes = STRUCT_OFFSET(FKeyData, DataPtr);

	template<class TKeyDataA, class TKeyDataB>
	friend FORCEINLINE bool operator==(const TKeyDataA& A, const TKeyDataB& B)
	{
		if (A.StrLen == B.StrLen)
		{
			if (A.StrLen >= 0)
			{
				// We can use Memcmp here as we know we're comparing two blocks of the same size and don't care about lexical ordering
				return FMemory::Memcmp(A.DataPtr, B.DataPtr, A.StrLen * sizeof(FTextKeyCharType)) == 0;
			}
				
			return A.AsGuid() == B.AsGuid();
		}
		return false;
	}

	void FindOrAddImpl(FStringView InStr, const uint32 InStrHash, FTextKey& OutTextKey)
	{
		int32 Index = INDEX_NONE;
		{
#if UE_TEXTKEY_SPLIT_GUID
			auto IsCompatibleGuidString =
				[InStr](FGuid& OutGuid)
				{
					// Only checking for EGuidFormats::Digits as that's the default of FGuid::ToString() as used by FText keys
					if (FGuid::ParseExact(InStr, EGuidFormats::Digits, OutGuid))
					{
						// Secondary check that every digit character parsed was uppercase, as ParseExact doesn't separate 
						// the parsing logic for Digits and DigitsLowercase, but we only optimize for uppercase as we need 
						// the String -> TextKey -> String conversion to be lossless for case
						// We could support other GUID formats if needed, but it would mean tracking what kind of GUID
						// was parsed so that it could be reconstructed corectly when used with FGuid::ToString
						for (const TCHAR GuidChar : InStr)
						{
							if (!FChar::IsDigit(GuidChar) && !FChar::IsUpper(GuidChar))
							{
								return false;
							}
						}
						return true;
					}
					return false;
				};

			FGuid KeyGuid;
			if (IsCompatibleGuidString(KeyGuid))
			{
				const FKeyDataView KeyData = FKeyDataView(KeyGuid, InStrHash);
				Index = FindOrAddData(KeyData);
			}
			else
#endif
			{
				auto ConvertedString = StrCast<FTextKeyCharType>(InStr.GetData(), InStr.Len());
				const FKeyDataView KeyData = FKeyDataView(FTextKeyStringView(ConvertedString.Get(), ConvertedString.Length()), InStrHash);
				Index = FindOrAddData(KeyData);
			}
		}
		check(Index != INDEX_NONE);

		OutTextKey.Index = Index;
#if UE_TEXTKEY_STORE_EMBEDDED_HASH
		OutTextKey.StrHash = InStrHash;
#endif
	}

	int32 FindOrAddData(const FKeyDataView& KeyData)
	{
		// Read-only
		int32 NumElementsOnRead = 0;
		{
			UE::TReadScopeLock ScopeLock(DataRW);

			if (int32 FoundIndex = KeyDataAllocations.Find(KeyData);
				FoundIndex != INDEX_NONE)
			{
				return FoundIndex;
			}

			NumElementsOnRead = KeyDataAllocations.Num();
		}

		// Write
		{
			UE::TWriteScopeLock ScopeLock(DataRW);

			if (KeyDataAllocations.Num() > NumElementsOnRead)
			{
				// Find again in case another thread beat us to it!
				if (int32 FoundIndex = KeyDataAllocations.Find(KeyData);
					FoundIndex != INDEX_NONE)
				{
					return FoundIndex;
				}
			}

			LLM_SCOPE_BYNAME(TEXT("Localization/TextKeys"));
			if (KeyData.IsStringView())
			{
				return KeyDataAllocations.Add(KeyData.AsStringView(), KeyData.StrHash);
			}
			else
			{
				return KeyDataAllocations.Add(KeyData.AsGuid(), KeyData.StrHash);
			}
		}
	}

	class FKeyDataAllocator
	{
	public:
		FKeyDataAllocator() = default;

		~FKeyDataAllocator()
		{
			FreeHash();
		}

		int32 Add(const FTextKeyStringView& InKeyStringView, uint32 StrHash)
		{
			const uint32 DataSizeBytes = InKeyStringView.Len() * sizeof(FTextKeyCharType);
#if UE_TEXTKEY_USE_INLINE_ALLOCATOR
			checkf(DataSizeBytes <= UE_TEXTKEY_MAX_SIZE_BYTES, TEXT("Key string size greater than UE_TEXTKEY_MAX_SIZE_BYTES"));
#endif
			
			const int32 NewElementIndex = Add(DataSizeBytes, StrHash);
			FKeyData& KeyData = Elements.Get(NewElementIndex);

#if UE_TEXTKEY_USE_INLINE_ALLOCATOR
			KeyData.StrLen = static_cast<int16>(InKeyStringView.Len());
			FMemory::Memcpy(KeyData.DataPtr, InKeyStringView.GetData(), DataSizeBytes);
#else
			KeyData.StrLen = InKeyStringView.Len();
			
			void* DataBuffer = GetPersistentLinearAllocator().Allocate(DataSizeBytes, alignof(FTextKeyCharType));
			FMemory::Memcpy(DataBuffer, InKeyStringView.GetData(), DataSizeBytes);
			KeyData.DataPtr = DataBuffer;
#endif
			
			return NewElementIndex;
		}
		
		int32 Add(const FGuid& InKeyGuid, uint32 StrHash)
		{
			constexpr uint32 DataSizeBytes = sizeof(FGuid);
			const int32 NewElementIndex = Add(DataSizeBytes, StrHash);

			FKeyData& KeyData = Elements.Get(NewElementIndex);
			KeyData.StrLen = INDEX_NONE;
#if UE_TEXTKEY_USE_INLINE_ALLOCATOR
			FGuid* GuidPtr = reinterpret_cast<FGuid*>(KeyData.DataPtr);
#else
			FGuid* GuidPtr = static_cast<FGuid*>(GetPersistentLinearAllocator().Allocate(sizeof(FGuid), alignof(FGuid)));
			KeyData.DataPtr = GuidPtr;
#endif
			*GuidPtr = InKeyGuid;
			
			return NewElementIndex;
		}

		int32 Find(const FKeyDataView& InKeyData) const
		{
			if (HashSize > 0)
			{
				const uint32 KeyDataHash = InKeyData.StrHash;
				const uint32 HashIndex = KeyDataHash & (HashSize - 1);

				int32 ElementIndex = Hash[HashIndex];
				while (ElementIndex != INDEX_NONE)
				{
					const FKeyData& Element = Elements.Get(ElementIndex);
					if (Element == InKeyData)
					{
						return ElementIndex;
					}
					ElementIndex = Element.NextElementIndex;
				}
			}
			return INDEX_NONE;
		}

		const FKeyData& Get(int32 InIndex) const
		{
			return Elements.Get(InIndex);
		}

		int32 Num() const
		{
			return Elements.Num();
		}

	private:
		void AllocateHash()
		{
			Hash = static_cast<int32*>(FMemory::Malloc(HashSize * sizeof(int32)));
			for (uint32 HashIndex = 0; HashIndex < HashSize; ++HashIndex)
			{
				Hash[HashIndex] = INDEX_NONE;
			}
		}

		void FreeHash()
		{
			FMemory::Free(Hash);
			Hash = nullptr;
		}

		void ConditionalRehash(const int32 NumElements)
		{
			const uint32 NewHashSize = FMath::Max<uint32>(MinHashSize, FPlatformMath::RoundUpToPowerOfTwo(NumElements / DEFAULT_NUMBER_OF_ELEMENTS_PER_HASH_BUCKET));
			if (NewHashSize > HashSize)
			{
				FreeHash();
				HashSize = NewHashSize;
				AllocateHash();

				int32 ElementIndex = Elements.GetFirstIndex();
				for (uint32 ElementCount = 0; ElementCount < Elements.Num(); ++ElementCount)
				{
					FKeyData& Element = Elements.Get(ElementIndex);
					
					const uint32 KeyDataHash = Element.StrHash;
					const uint32 HashIndex = KeyDataHash & (HashSize - 1);

					Element.NextElementIndex = Hash[HashIndex];
					Hash[HashIndex] = ElementIndex;
					
					ElementIndex = Elements.GetNextIndex(ElementIndex, GetAllocationSize(Element));
				}
			}
		}

		static uint32 GetAllocationSize(const FKeyData& KeyData)
		{
			uint32 DataSize;
			if (KeyData.IsStringView())
			{
				DataSize = KeyData.StrLen * sizeof(FTextKeyCharType);
			}
			else
			{
				DataSize = sizeof(FGuid);
			}
			return GetAllocationSize(DataSize);
		}

		static uint32 GetAllocationSize(const uint32 DataSize)
		{
#if UE_TEXTKEY_USE_INLINE_ALLOCATOR
			return Align(KeyDataHeaderSizeBytes + DataSize, alignof(FKeyData)); 
#else
			return sizeof(FKeyData);
#endif
		}

		int32 Add(const uint32 DataSize, uint32 StrHash)
		{
			ConditionalRehash(Elements.Num() + 1);

			const uint32 HashIndex = StrHash & (HashSize - 1);
			const int32 NewElementIndex = Elements.Add(GetAllocationSize(DataSize));
			
			FKeyData& NewElement = Elements.Get(NewElementIndex);
			NewElement.StrHash = StrHash;
			NewElement.NextElementIndex = Hash[HashIndex];
			Hash[HashIndex] = NewElementIndex;
			
			return NewElementIndex;
		}

		/** Values; indices are referenced by the hash and FTextKey */
		TChunkedPackedArray<FKeyData> Elements;

		// Add in a bit of overhead for the FKeyData header
		static_assert(UE_TEXTKEY_MAX_SIZE_BYTES + KeyDataHeaderSizeBytes <= TChunkedPackedArray<FKeyData>::BytesPerChunk,
			"UE_TEXTKEY_MAX_SIZE_BYTES is too large for the storage array chunk size");

		static constexpr int32 MinHashSize = UE_TEXTKEY_ELEMENTS_MIN_HASH_SIZE;
		/** Current size of the hash; if this changes the hash must be rebuilt */
		uint32 HashSize = 0;
		/** Index of the root element in each hash bucket; use FElement::NextElementIndex to walk the bucket */
		int32* Hash = nullptr;
	};

	mutable FTransactionallySafeRWLock DataRW;
	FKeyDataAllocator KeyDataAllocations;
};

namespace TextKeyUtil
{

static constexpr int32 InlineStringSize = 128;
using FInlineStringBuffer = TArray<TCHAR, TInlineAllocator<InlineStringSize>>;
using FInlineStringBuilder = TStringBuilder<InlineStringSize>;

static_assert(PLATFORM_LITTLE_ENDIAN, "FTextKey serialization needs updating to support big-endian platforms!");

bool SaveKeyString(FArchive& Ar, const TCHAR* InStrPtr)
{
	// Note: This serialization should be compatible with the FString serialization, but avoids creating an FString if the FTextKey is already cached
	// > 0 for ANSICHAR, < 0 for UTF16CHAR serialization
	check(!Ar.IsLoading());

	const bool bSaveUnicodeChar = Ar.IsForcingUnicode() || !FCString::IsPureAnsi(InStrPtr);
	if (bSaveUnicodeChar)
	{
		// Note: This is a no-op on platforms that are using a 16-bit TCHAR
		FTCHARToUTF16 UTF16String(InStrPtr);
		const int32 Num = UTF16String.Length() + 1; // include the null terminator

		int32 SaveNum = -Num;
		Ar << SaveNum;

		if (Num)
		{
			Ar.Serialize((void*)UTF16String.Get(), sizeof(UTF16CHAR) * Num);
		}
	}
	else
	{
		int32 Num = FCString::Strlen(InStrPtr) + 1; // include the null terminator
		Ar << Num;

		if (Num)
		{
			Ar.Serialize((void*)StringCast<ANSICHAR>(InStrPtr, Num).Get(), sizeof(ANSICHAR) * Num);
		}
	}

	return true;
}

bool LoadKeyString(FArchive& Ar, FInlineStringBuffer& OutStrBuffer)
{
	// Note: This serialization should be compatible with the FString serialization, but avoids creating an FString if the FTextKey is already cached
	// > 0 for ANSICHAR, < 0 for UTF16CHAR serialization
	check(Ar.IsLoading());

	int32 SaveNum = 0;
	Ar << SaveNum;

	const bool bLoadUnicodeChar = SaveNum < 0;
	if (bLoadUnicodeChar)
	{
		SaveNum = -SaveNum;
	}

	// If SaveNum is still less than 0, they must have passed in MIN_INT. Archive is corrupted.
	if (SaveNum < 0)
	{
		Ar.SetCriticalError();
		return false;
	}

	// Protect against network packets allocating too much memory
	const int64 MaxSerializeSize = Ar.GetMaxSerializeSize();
	if ((MaxSerializeSize > 0) && (SaveNum > MaxSerializeSize))
	{
		Ar.SetCriticalError();
		return false;
	}

	// Create a buffer of the correct size
	OutStrBuffer.AddUninitialized(SaveNum);

	if (SaveNum)
	{
		if (bLoadUnicodeChar)
		{
			// Read in the Unicode string
			auto Passthru = StringMemoryPassthru<UCS2CHAR, TCHAR, InlineStringSize>(OutStrBuffer.GetData(), SaveNum, SaveNum);
			Ar.Serialize(Passthru.Get(), SaveNum * sizeof(UCS2CHAR));
			Passthru.Get()[SaveNum - 1] = 0; // Ensure the string has a null terminator
			Passthru.Apply();
		}
		else
		{
			// Read in the ANSI string
			auto Passthru = StringMemoryPassthru<ANSICHAR, TCHAR, InlineStringSize>(OutStrBuffer.GetData(), SaveNum, SaveNum);
			Ar.Serialize(Passthru.Get(), SaveNum * sizeof(ANSICHAR));
			Passthru.Get()[SaveNum - 1] = 0; // Ensure the string has a null terminator
			Passthru.Apply();
		}

		UE_CLOGF(SaveNum > InlineStringSize, LogTextKey, VeryVerbose, "Key string '%ls' was larger (%d) than the inline size (%d) and caused an allocation!", OutStrBuffer.GetData(), SaveNum, InlineStringSize);
	}

	return true;
}

uint32 HashString(const FTCHARToUTF16& InStr)
{
	const uint64 StrHash = CityHash64((const char*)InStr.Get(), InStr.Length() * sizeof(UTF16CHAR));
	return GetTypeHash(StrHash);
}

}

FTextKey::FTextKey(FStringView InStr)
{
	if (InStr.IsEmpty())
	{
		Reset();
	}
	else
	{
		FTextKeyState::GetState().FindOrAdd(InStr, *this);
	}
}

FTextKey::FTextKey(const TCHAR* InStr)
	: FTextKey(FStringView(InStr))
{
}

FTextKey::FTextKey(const FString& InStr)
	: FTextKey(FStringView(InStr))
{
}

FString FTextKey::ToString() const
{
	FString Out;
	AppendString(Out);
	return Out;
}

void FTextKey::ToString(FString& Out) const
{
	Out.Reset();
	AppendString(Out);
}

void FTextKey::ToString(FStringBuilderBase& Out) const
{
	Out.Reset();
	AppendString(Out);
}

void FTextKey::AppendString(FString& Out) const
{
	if (Index != INDEX_NONE)
	{
		FTextKeyState::GetState().AppendStringByIndex(Index, Out);
	}
}

void FTextKey::AppendString(FStringBuilderBase& Out) const
{
	if (Index != INDEX_NONE)
	{
		FTextKeyState::GetState().AppendStringByIndex(Index, Out);
	}
}

uint32 GetTypeHash(const FTextKey& A)
{
#if UE_TEXTKEY_STORE_EMBEDDED_HASH
	return A.StrHash;
#else
	return A.Index != INDEX_NONE
		? FTextKeyState::GetState().GetHashByIndex(A.Index)
		: 0;
#endif
}

void FTextKey::SerializeAsString(FArchive& Ar)
{
	if (Ar.IsLoading())
	{
		TextKeyUtil::FInlineStringBuffer StrBuffer;
		TextKeyUtil::LoadKeyString(Ar, StrBuffer);

		if (StrBuffer.Num() <= 1)
		{
			Reset();
		}
		else
		{
			FTextKeyState::GetState().FindOrAdd(FStringView(StrBuffer.GetData(), StrBuffer.Num() - 1), *this);
		}
	}
	else
	{
		TextKeyUtil::FInlineStringBuilder StrBuilder;
		AppendString(StrBuilder);
		TextKeyUtil::SaveKeyString(Ar, *StrBuilder);
	}
}

void FTextKey::SerializeWithHash(FArchive& Ar)
{
	if (Ar.IsLoading())
	{
		uint32 TmpStrHash = 0;
		Ar << TmpStrHash;

		TextKeyUtil::FInlineStringBuffer StrBuffer;
		TextKeyUtil::LoadKeyString(Ar, StrBuffer);

		if (StrBuffer.Num() <= 1)
		{
			Reset();
		}
		else
		{
			FTextKeyState::GetState().FindOrAdd(FStringView(StrBuffer.GetData(), StrBuffer.Num() - 1), TmpStrHash, *this);
		}
	}
	else
	{
		uint32 TmpStrHash = GetTypeHash(*this);
		Ar << TmpStrHash;

		TextKeyUtil::FInlineStringBuilder StrBuilder;
		AppendString(StrBuilder);
		TextKeyUtil::SaveKeyString(Ar, *StrBuilder);
	}
}

void FTextKey::SerializeDiscardHash(FArchive& Ar)
{
	if (Ar.IsLoading())
	{
		uint32 DiscardedHash = 0;
		Ar << DiscardedHash;

		TextKeyUtil::FInlineStringBuffer StrBuffer;
		TextKeyUtil::LoadKeyString(Ar, StrBuffer);

		if (StrBuffer.Num() <= 1)
		{
			Reset();
		}
		else
		{
			FTextKeyState::GetState().FindOrAdd(FStringView(StrBuffer.GetData(), StrBuffer.Num() - 1), *this);
		}
	}
	else
	{
		uint32 TmpStrHash = GetTypeHash(*this);
		Ar << TmpStrHash;

		TextKeyUtil::FInlineStringBuilder StrBuilder;
		AppendString(StrBuilder);
		TextKeyUtil::SaveKeyString(Ar, *StrBuilder);
	}
}

void FTextKey::SerializeAsString(FStructuredArchiveSlot Slot)
{
	if (Slot.GetArchiveState().IsTextFormat())
	{
		if (Slot.GetUnderlyingArchive().IsLoading())
		{
			FString TmpStr;
			Slot << TmpStr;

			if (TmpStr.IsEmpty())
			{
				Reset();
			}
			else
			{
				FTextKeyState::GetState().FindOrAdd(TmpStr, *this);
			}
		}
		else
		{
			FString TmpStr = ToString();
			Slot << TmpStr;
		}
	}
	else
	{
		Slot.EnterStream(); // Let the slot know that we will custom serialize
		SerializeAsString(Slot.GetUnderlyingArchive());
	}
}

void FTextKey::SerializeWithHash(FStructuredArchiveSlot Slot)
{
	if (Slot.GetArchiveState().IsTextFormat())
	{
		FStructuredArchiveRecord Record = Slot.EnterRecord();

		if (Slot.GetUnderlyingArchive().IsLoading())
		{
			uint32 TmpStrHash = 0;
			Record << SA_VALUE(TEXT("Hash"), TmpStrHash);

			FString TmpStr;
			Record << SA_VALUE(TEXT("Str"), TmpStr);

			if (TmpStr.IsEmpty())
			{
				Reset();
			}
			else
			{
				FTextKeyState::GetState().FindOrAdd(TmpStr, TmpStrHash, *this);
			}
		}
		else
		{
			uint32 TmpStrHash = GetTypeHash(*this);
			Record << SA_VALUE(TEXT("Hash"), TmpStrHash);

			FString TmpStr = ToString();
			Record << SA_VALUE(TEXT("Str"), TmpStr);
		}
	}
	else
	{
		Slot.EnterStream(); // Let the slot know that we will custom serialize
		SerializeWithHash(Slot.GetUnderlyingArchive());
	}
}

void FTextKey::SerializeDiscardHash(FStructuredArchiveSlot Slot)
{
	if (Slot.GetArchiveState().IsTextFormat())
	{
		FStructuredArchiveRecord Record = Slot.EnterRecord();

		if (Slot.GetUnderlyingArchive().IsLoading())
		{
			uint32 DiscardedHash = 0;
			Record << SA_VALUE(TEXT("Hash"), DiscardedHash);

			FString TmpStr;
			Record << SA_VALUE(TEXT("Str"), TmpStr);

			if (TmpStr.IsEmpty())
			{
				Reset();
			}
			else
			{
				FTextKeyState::GetState().FindOrAdd(TmpStr, *this);
			}
		}
		else
		{
			uint32 TmpStrHash = GetTypeHash(*this);
			Record << SA_VALUE(TEXT("Hash"), TmpStrHash);

			FString TmpStr = ToString();
			Record << SA_VALUE(TEXT("Str"), TmpStr);
		}
	}
	else
	{
		Slot.EnterStream(); // Let the slot know that we will custom serialize
		SerializeDiscardHash(Slot.GetUnderlyingArchive());
	}
}

void FTextKey::CompactDataStructures()
{
	FTextKeyState::GetState().Shrink();
}

void FTextKey::TearDown()
{
	FTextKeyState::TearDown();
}
