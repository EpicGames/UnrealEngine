// Copyright Epic Games, Inc. All Rights Reserved.

#include "Internationalization/TextLocalizationManager.h"
#include "Internationalization/TextLocalizationResource.h"
#include "Internationalization/ILocalizedTextSource.h"
#include "Internationalization/LocalizationResourceTextSource.h"
#include "Internationalization/PolyglotTextSource.h"
#include "Internationalization/StringTableRegistry.h"
#include "Internationalization/Cultures/LeetCulture.h"
#include "Internationalization/Cultures/KeysCulture.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/FileManager.h"
#include "HAL/LowLevelMemTracker.h"
#include "HAL/ThreadHeartBeat.h"
#include "Misc/FileHelper.h"
#include "Misc/Parse.h"
#include "Misc/ScopeLock.h"
#include "Containers/Ticker.h"
#include "Misc/ScopeRWLock.h"
#include "Misc/CommandLine.h"
#include "Misc/LazySingleton.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Internationalization/Culture.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/StringTableCore.h"
#include "Internationalization/TextNamespaceUtil.h"
#include "Internationalization/TextCache.h"
#include "Stats/Stats.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/App.h"
#include "Misc/CoreDelegates.h"
#include "Templates/UniquePtr.h"
#include "Async/Async.h"
#include "Async/TaskGraphInterfaces.h"

DEFINE_LOG_CATEGORY_STATIC(LogTextLocalizationManager, Log, All);

namespace TextLocalizationManager
{
enum class EDisplayStringSupport : int32
{
	Auto = 0,
	Enabled = 1,
	Disabled = 2,
};
static int32 DisplayStringSupport = static_cast<int32>(EDisplayStringSupport::Auto);
static FAutoConsoleVariableRef CVarDisplayStringSupport(TEXT("Localization.DisplayStringSupport"), DisplayStringSupport, TEXT("Is display string support enabled? 0: Auto (default), 1: Enabled, 2: Disabled"));

static bool AsyncLoadLocalizationData = true;
static FAutoConsoleVariableRef CVarAsyncLoadLocalizationData(TEXT("Localization.AsyncLoadLocalizationData"), AsyncLoadLocalizationData, TEXT("True to load localization data asynchronously (non-blocking), or False to load it synchronously (blocking)"));

/**
 * Note: This is disabled by default because we have existing code that conflates "the language changed" with "there is new localization data available".
 * These places should be audited to replace FInternationalization::OnCultureChanged callbacks with FTextLocalizationManager::OnTextRevisionChangedEvent.
 * 
 * The most troublesome place to handle is the font and composite font caches, which flush on a language change (because the language can affect which fonts 
 * will be used, and is generally a good point to clear any current font cache data), which results in the font cache re-filling with glyphs for the previous 
 * language (as the new localization data hasn't loaded yet). Additionally composite fonts with per-language sub-fonts can briefly show text using the wrong 
 * set of glyphs (eg, when switching from "ja" -> "zh-Hans" you'll see Japanese text using Chinese-style Han before the Chinese text loads in).
 * 
 * We don't want to flush these font caches every time new localization data is available, so we'll likely need extra context in OnTextRevisionChangedEvent 
 * to know what caused the data to change (eg, LanguageChanged, RefreshRequested, AdditionalDataLoaded, etc) so that the callback can behave accordingly.
 */
static bool AsyncLoadLocalizationDataOnLanguageChange = false;
static FAutoConsoleVariableRef CVarAsyncLoadLocalizationDataOnLanguageChange(TEXT("Localization.AsyncLoadLocalizationDataOnLanguageChange"), AsyncLoadLocalizationDataOnLanguageChange, TEXT("True to load localization data asynchronously (non-blocking) when the language changes, or False to load it synchronously (blocking)"));

static bool AlwaysLoadNativeLocalizationDataDuringInitialization = false;
static FAutoConsoleVariableRef CVarAlwaysLoadNativeLocalizationDataDuringInitialization(TEXT("Localization.AlwaysLoadNativeLocalizationDataDuringInitialization"), AlwaysLoadNativeLocalizationDataDuringInitialization, TEXT("True to load the native localization data during initialization, even if we're not starting in the native language. This ensures that all gathered text will load some localization data, even if not fully translated."));

#if WITH_EDITOR
static bool ForceLoadGameLocalizationInEditor = false;
static FAutoConsoleVariableRef CVarForceLoadGameLocalizationInEditor(TEXT("Localization.ForceLoadGameLocalizationInEditor"), ForceLoadGameLocalizationInEditor, TEXT("True to force load game localization data in an editor"));
#endif

#if ENABLE_LOC_TESTING
static FAutoConsoleCommand CmdDumpLiveTable(
	TEXT("Localization.DumpLiveTable"), 
	TEXT("Dumps the current live table state to the log, optionally filtering it based on wildcard arguments for 'Namespace', 'Key', or 'DisplayString', eg) -Key=Foo, or -DisplayString=\"This is some text\", or -Key=Bar*Baz -DisplayString=\"This is some other text\""), 
	FConsoleCommandWithArgsDelegate::CreateStatic([](const TArray<FString>& Args)
	{
		// Rebuild the full string of arguments, since values within quotes may have been split on spaces
		const FString Arguments = FString::Join(Args, TEXT(" "));

		auto ParseOptionalStringArg = [](const TCHAR* Arg, const TCHAR* TokenName, TOptional<FString>& OutResult)
		{
			FString TmpResult;
			if (FParse::Value(Arg, TokenName, TmpResult))
			{
				OutResult = MoveTemp(TmpResult);
				return true;
			}
			return false;
		};

		TOptional<FString> NamespaceFilter;
		TOptional<FString> KeyFilter;
		TOptional<FString> DisplayStringFilter;
		TOptional<FString> DumpFile;

		if (!ParseOptionalStringArg(*Arguments, TEXT("Namespace="), NamespaceFilter) &&
			!ParseOptionalStringArg(*Arguments, TEXT("Key="), KeyFilter) &&
			!ParseOptionalStringArg(*Arguments, TEXT("DisplayString="), DisplayStringFilter) && 
			!ParseOptionalStringArg(*Arguments, TEXT("DumpFile="), DumpFile))
		{
			UE_LOGF(LogLocalization, Warning, "Unknown argument passed to Localization.DumpLiveTable!");
		}

		// Block dumping all 500k strings which may crash the editor
		if (!NamespaceFilter.IsSet() && !KeyFilter.IsSet() && !DisplayStringFilter.IsSet() && !DumpFile.IsSet())
		{
			UE_LOGF(LogLocalization, Display, "No arguments provided, this would dump every string. Consider dumping to a file instead, or providing filter(s) for Namespace, Key, or DisplayString");
			return;
		}

		if (DumpFile.IsSet())
		{
			FTextLocalizationManager::Get().DumpLiveTable(DumpFile.GetValue(), NamespaceFilter.GetPtrOrNull(), KeyFilter.GetPtrOrNull(), DisplayStringFilter.GetPtrOrNull());
		}
		else
		{
#if !NO_LOGGING
			FTextLocalizationManager::Get().DumpLiveTable(NamespaceFilter.GetPtrOrNull(), KeyFilter.GetPtrOrNull(), DisplayStringFilter.GetPtrOrNull(), &LogConsoleResponse);
#endif
		}
	}));

static FAutoConsoleCommand SetDisplayString(
	TEXT("Localization.SetDisplayString"),
	TEXT("Add/Update DisplayString in the live table given required arguments: 'Namespace', 'Key', and 'DisplayString'. 'SourceString' is optional, but needed if adding a new display string."),
	FConsoleCommandWithArgsDelegate::CreateStatic([](const TArray<FString>& Args)
	{
		// Rebuild the full string of arguments, since values within quotes may have been split on spaces
		const FString Arguments = FString::Join(Args, TEXT(" "));

		FString Namespace;
		TOptional<FString> Key;
		TOptional<FString> DisplayString;
		TOptional<FString> SourceString;

		FString Tmp;
		if (FParse::Value(*Arguments, TEXT("Namespace="), Tmp))
		{
			Namespace = MoveTemp(Tmp);
		}

		if (FParse::Value(*Arguments, TEXT("Key="), Tmp))
		{
			Key = MoveTemp(Tmp);
		}

		if (FParse::Value(*Arguments, TEXT("DisplayString="), Tmp))
		{
			DisplayString = MoveTemp(Tmp);
		}

		if (FParse::Value(*Arguments, TEXT("SourceString="), Tmp))
		{
			SourceString = MoveTemp(Tmp);
		}

		// Namespace is optional, assumed to be empty if not provided
		if (!Key.IsSet() || !DisplayString.IsSet())
		{
			UE_LOGF(LogLocalization, Display, "Missing argument(s): Key and/or DisplayString");
			return;
		}
		// An empty DisplayString is allowed, but the argument for it must be provided: -DisplayString=""
		if (Key.GetValue().IsEmpty())
		{
			UE_LOGF(LogLocalization, Display, "Empty argument: Key");
			return;
		}

		FTextLocalizationManager::Get().AddOrUpdateDisplayStringInLiveTable(Namespace, Key.GetValue(), DisplayString.GetValue(), SourceString.GetPtrOrNull());

		// For live game, force widget invalidation to update the text onscreen. This isn't needed in the editor.
		if (IConsoleObject* CObj = IConsoleManager::Get().FindConsoleObject(TEXT("Slate.TriggerInvalidate")))
		{
			CObj->AsCommand()->Execute(/*Args=*/TArray<FString>(), /*InWorld=*/nullptr, *GLog);
		}
	}));
#endif

FString KeyifyTextId(const FTextId& TextId)
{
	// We want to show the identity in terms of key, namespace. This is to try and fit into the constraints of UI text blocks and at least let the key component be visible to easily identify a piece of text.
	// If the key/namespace pair is too long, the Slate.LogPaintedText cvar can be used to see the entire thing.
	return FString::Printf(TEXT("%s, %s"), *TextId.GetKey().ToString(), *TextId.GetNamespace().ToString());
}

FString ConditionDisplayStringForLog(const FString& DisplayString)
{
	static const TArray<TCHAR> CharsToReplace = { TEXT('\n'), TEXT('\r'), TEXT('\t'), TEXT('"'), TEXT('\\') };
	return DisplayString.ReplaceCharWithEscapedChar(&CharsToReplace);
}

}

enum class ERequestedCultureOverrideLevel : uint8
{
	CommandLine,
	EditorSettings,
	GameUserSettings,
	GameSettings,
	EngineSettings,
	Defaults,
};

bool IsLocalizationLockedByConfig()
{
	bool bIsLocalizationLocked = false;
	if (!GConfig->GetBool(TEXT("Internationalization"), TEXT("LockLocalization"), bIsLocalizationLocked, GGameIni))
	{
		GConfig->GetBool(TEXT("Internationalization"), TEXT("LockLocalization"), bIsLocalizationLocked, GEngineIni);
	}
	return bIsLocalizationLocked;
}

FString GetRequestedCulture(const TCHAR* InCommandLineKey, const TCHAR* InConfigKey, const TCHAR* InDefaultCulture, ERequestedCultureOverrideLevel& OutOverrideLevel)
{
	FString RequestedCulture;

	auto ReadSettingsFromCommandLine = [&RequestedCulture, &InCommandLineKey, &InConfigKey, &OutOverrideLevel]()
	{
#if ENABLE_LOC_TESTING
		if (RequestedCulture.IsEmpty() && FParse::Param(FCommandLine::Get(), *FLeetCulture::StaticGetName()))
		{
			RequestedCulture = FLeetCulture::StaticGetName();
			OutOverrideLevel = ERequestedCultureOverrideLevel::CommandLine;
		}
		else if (RequestedCulture.IsEmpty() && FParse::Param(FCommandLine::Get(), *FKeysCulture::StaticGetName()))
		{
			RequestedCulture = FKeysCulture::StaticGetName();
			OutOverrideLevel = ERequestedCultureOverrideLevel::CommandLine;
		}
#endif

		if (RequestedCulture.IsEmpty() && FParse::Value(FCommandLine::Get(), TEXT("CULTUREFORCOOKING="), RequestedCulture))
		{
			OutOverrideLevel = ERequestedCultureOverrideLevel::CommandLine;

			// Write the culture passed in if first install...
			if (FParse::Param(FCommandLine::Get(), TEXT("firstinstall")) && InConfigKey)
			{
				GConfig->SetString(TEXT("Internationalization"), InConfigKey, *RequestedCulture, GEngineIni);
			}
		}

		if (RequestedCulture.IsEmpty() && InCommandLineKey && FParse::Value(FCommandLine::Get(), InCommandLineKey, RequestedCulture))
		{
			OutOverrideLevel = ERequestedCultureOverrideLevel::CommandLine;
		}

		if (RequestedCulture.IsEmpty() && FParse::Value(FCommandLine::Get(), TEXT("CULTURE="), RequestedCulture))
		{
			OutOverrideLevel = ERequestedCultureOverrideLevel::CommandLine;
		}
	};

	auto ReadSettingsFromConfig = [&RequestedCulture, &InConfigKey, &OutOverrideLevel](const FString& InConfigFilename, const ERequestedCultureOverrideLevel InConfigOverrideLevel)
	{
		if (RequestedCulture.IsEmpty() && InConfigKey && GConfig->GetString(TEXT("Internationalization"), InConfigKey, RequestedCulture, InConfigFilename))
		{
			OutOverrideLevel = InConfigOverrideLevel;
		}

		if (RequestedCulture.IsEmpty() && GConfig->GetString(TEXT("Internationalization"), TEXT("Culture"), RequestedCulture, InConfigFilename))
		{
			OutOverrideLevel = InConfigOverrideLevel;
		}
	};

	auto ReadSettingsFromDefaults = [&RequestedCulture, &InDefaultCulture, &OutOverrideLevel]()
	{
		if (RequestedCulture.IsEmpty() && InDefaultCulture)
		{
			RequestedCulture = InDefaultCulture;
			OutOverrideLevel = ERequestedCultureOverrideLevel::Defaults;
		}
	};

	// Read setting override specified on commandline.
	ReadSettingsFromCommandLine();
#if WITH_EDITOR
	// Read setting specified in editor configuration.
	if (GIsEditor)
	{
		ReadSettingsFromConfig(GEditorSettingsIni, ERequestedCultureOverrideLevel::EditorSettings);
	}
#endif // WITH_EDITOR
	// Read setting specified in game configurations.
	if (!GIsEditor)
	{
		ReadSettingsFromConfig(GGameUserSettingsIni, ERequestedCultureOverrideLevel::GameUserSettings);
		ReadSettingsFromConfig(GGameIni, ERequestedCultureOverrideLevel::GameSettings);
	}
	// Read setting specified in engine configuration.
	ReadSettingsFromConfig(GEngineIni, ERequestedCultureOverrideLevel::EngineSettings);
	// Read defaults
	ReadSettingsFromDefaults();

	return RequestedCulture;
}

FString GetRequestedLanguage(ERequestedCultureOverrideLevel& OutOverrideLevel)
{
	return GetRequestedCulture(TEXT("LANGUAGE="), TEXT("Language"), *FInternationalization::Get().GetDefaultLanguage()->GetName(), OutOverrideLevel);
}

FString GetRequestedLocale(ERequestedCultureOverrideLevel& OutOverrideLevel)
{
	return GetRequestedCulture(TEXT("LOCALE="), TEXT("Locale"), *FInternationalization::Get().GetDefaultLocale()->GetName(), OutOverrideLevel);
}

TArray<TTuple<FName, FString>> GetRequestedAssetGroups(const ERequestedCultureOverrideLevel InLanguageOverrideLevel)
{
	TArray<TTuple<FName, FString>> RequestedAssetGroups;

	auto ReadSettingsFromConfig = [&RequestedAssetGroups, &InLanguageOverrideLevel](const FString& InConfigFilename, const ERequestedCultureOverrideLevel InConfigOverrideLevel)
	{
		// Once the language has been overridden we stop parsing out new asset groups
		if (InLanguageOverrideLevel <= InConfigOverrideLevel)
		{
			if (const FConfigSection* AssetGroupCulturesSection = GConfig->GetSection(TEXT("Internationalization.AssetGroupCultures"), false, InConfigFilename))
			{
				for (const auto& SectionEntryPair : *AssetGroupCulturesSection)
				{
					const bool bAlreadyExists = RequestedAssetGroups.ContainsByPredicate([&](const TTuple<FName, FString>& InRequestedAssetGroup)
					{
						return InRequestedAssetGroup.Key == SectionEntryPair.Key;
					});

					if (!bAlreadyExists)
					{
						RequestedAssetGroups.Add(MakeTuple(SectionEntryPair.Key, SectionEntryPair.Value.GetValue()));
					}
				}
			}
		}
	};

#if WITH_EDITOR
	// Read setting specified in editor configuration.
	if (GIsEditor)
	{
		ReadSettingsFromConfig(GEditorSettingsIni, ERequestedCultureOverrideLevel::EditorSettings);
	}
#endif // WITH_EDITOR
	// Read setting specified in game configurations.
	if (!GIsEditor)
	{
		ReadSettingsFromConfig(GGameUserSettingsIni, ERequestedCultureOverrideLevel::GameUserSettings);
		ReadSettingsFromConfig(GGameIni, ERequestedCultureOverrideLevel::GameSettings);
	}
	// Read setting specified in engine configuration.
	ReadSettingsFromConfig(GEngineIni, ERequestedCultureOverrideLevel::EngineSettings);

	return RequestedAssetGroups;
}

void ApplyDefaultCultureSettings(const ELocalizationLoadFlags LocLoadFlags)
{
	FInternationalization& I18N = FInternationalization::Get();

	auto LogCultureOverride = [](const TCHAR* InResult, const TCHAR* InOptionDisplayName, const ERequestedCultureOverrideLevel InOverrideLevel)
	{
		switch (InOverrideLevel)
		{
		case ERequestedCultureOverrideLevel::CommandLine:
			UE_LOGF(LogInit, Log, "Overriding %ls with command-line option (%ls).", InOptionDisplayName, InResult);
			break;
		case ERequestedCultureOverrideLevel::EditorSettings:
			UE_LOGF(LogInit, Log, "Overriding language with editor %ls configuration option (%ls).", InOptionDisplayName, InResult);
			break;
		case ERequestedCultureOverrideLevel::GameUserSettings:
			UE_LOGF(LogInit, Log, "Overriding language with game user settings %ls configuration option (%ls).", InOptionDisplayName, InResult);
			break;
		case ERequestedCultureOverrideLevel::GameSettings:
			UE_LOGF(LogInit, Log, "Overriding language with game %ls configuration option (%ls).", InOptionDisplayName, InResult);
			break;
		case ERequestedCultureOverrideLevel::EngineSettings:
			UE_LOGF(LogInit, Log, "Overriding language with engine %ls configuration option (%ls).", InOptionDisplayName, InResult);
			break;
		case ERequestedCultureOverrideLevel::Defaults:
			UE_LOGF(LogInit, Log, "Using OS detected %ls (%ls).", InOptionDisplayName, InResult);
			break;
		}
	};

	auto ValidateRequestedCulture = [LocLoadFlags, &I18N](const FString& InRequestedCulture, const FString& InFallbackCulture, const TCHAR* InLogDesc, const bool bRequireExactMatch) -> FString
	{
		FString TargetCultureName = InRequestedCulture;

#if ENABLE_LOC_TESTING
		bool bIsTargetCultureDebugCulture = (TargetCultureName == FLeetCulture::StaticGetName()) || (TargetCultureName == FKeysCulture::StaticGetName());
		if (!bIsTargetCultureDebugCulture)
#endif
		{
			// Validate the locale has data or fallback to one that does.
			const TArray<FString> AvailableCultureNames = FTextLocalizationManager::Get().GetLocalizedCultureNames(LocLoadFlags);
			auto ValidateCultureName = [&AvailableCultureNames, &I18N](const FString& InCultureToValidate) -> FString
			{
				const TArray<FString> PrioritizedCultureNames = I18N.GetPrioritizedCultureNames(InCultureToValidate);
				for (const FString& CultureName : PrioritizedCultureNames)
				{
					if (AvailableCultureNames.Contains(CultureName))
					{
						return CultureName;
					}
				}
				return FString();
			};

			const FString ValidCultureName = ValidateCultureName(InRequestedCulture);
			const FString ValidFallbackCultureName = ValidateCultureName(InFallbackCulture);

			if (!ValidCultureName.IsEmpty())
			{
				if (bRequireExactMatch && InRequestedCulture != ValidCultureName)
				{
					TargetCultureName = ValidCultureName;
					UE_LOGF(LogTextLocalizationManager, Log, "No specific localization for '%ls' exists, so '%ls' will be used for the %ls.", *InRequestedCulture, *TargetCultureName, InLogDesc);
				}
			}
			else if (!ValidFallbackCultureName.IsEmpty())
			{
				TargetCultureName = ValidFallbackCultureName;
				UE_LOGF(LogTextLocalizationManager, Log, "No localization for '%ls' exists, so '%ls' will be used for the %ls.", *InRequestedCulture, *TargetCultureName, InLogDesc);
			}
			else
			{
				TargetCultureName = AvailableCultureNames.Num() > 0 ? AvailableCultureNames[0] : InFallbackCulture;
				UE_LOGF(LogTextLocalizationManager, Log, "No localization for '%ls' exists, so '%ls' will be used for the %ls.", *InRequestedCulture, *TargetCultureName, InLogDesc);
			}
		}

		return TargetCultureName;
	};

	FString FallbackLanguage = "en";
	if (EnumHasAnyFlags(LocLoadFlags, ELocalizationLoadFlags::Game))
	{
		// If this is a game, use the native culture of the game as the fallback
		FString NativeGameCulture = FTextLocalizationManager::Get().GetNativeCultureName(ELocalizedTextSourceCategory::Game);
		if (!NativeGameCulture.IsEmpty())
		{
			FallbackLanguage = MoveTemp(NativeGameCulture);
		}
	}

	FString RequestedLanguage;
	FString RequestedLocale;
	TArray<TTuple<FName, FString>> RequestedAssetGroups;
	{
		ERequestedCultureOverrideLevel LanguageOverrideLevel = ERequestedCultureOverrideLevel::Defaults;
		RequestedLanguage = GetRequestedLanguage(LanguageOverrideLevel);
		LogCultureOverride(*RequestedLanguage, TEXT("language"), LanguageOverrideLevel);

		ERequestedCultureOverrideLevel LocaleOverrideLevel = ERequestedCultureOverrideLevel::Defaults;
		RequestedLocale = GetRequestedLocale(LocaleOverrideLevel);
		LogCultureOverride(*RequestedLocale, TEXT("locale"), LocaleOverrideLevel);

		RequestedAssetGroups = GetRequestedAssetGroups(LanguageOverrideLevel);
	}

	// Validate that we have translations for this language and locale
	// Note: We skip the locale check for the editor as we a limited number of translations, but want to allow locale correct display of numbers, dates, etc
	const FString TargetLanguage = ValidateRequestedCulture(RequestedLanguage, FallbackLanguage, TEXT("language"), true);
	const FString TargetLocale = GIsEditor ? RequestedLocale : ValidateRequestedCulture(RequestedLocale, TargetLanguage, TEXT("locale"), false);
	if (TargetLanguage == TargetLocale)
	{
		I18N.SetCurrentLanguageAndLocale(TargetLanguage);
	}
	else
	{
		I18N.SetCurrentLanguage(TargetLanguage);
		I18N.SetCurrentLocale(TargetLocale);
	}

	for (const auto& RequestedAssetGroupPair : RequestedAssetGroups)
	{
		const FString TargetAssetGroupCulture = ValidateRequestedCulture(RequestedAssetGroupPair.Value, TargetLanguage, *FString::Printf(TEXT("'%s' asset group"), *RequestedAssetGroupPair.Key.ToString()), false);
		if (TargetAssetGroupCulture != TargetLanguage)
		{
			I18N.SetCurrentAssetGroupCulture(RequestedAssetGroupPair.Key, TargetAssetGroupCulture);
		}
	}
}

void BeginPreInitTextLocalization()
{
	LLM_SCOPE(ELLMTag::Localization);

	SCOPED_BOOT_TIMING("BeginPreInitTextLocalization");
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("BeginPreInitTextLocalization"), STAT_BeginPreInitTextLocalization, STATGROUP_LoadTime);

	// Bind this delegate before the PAK file loader is created
	FCoreDelegates::GetOnPakFileMounted2().AddRaw(&FTextLocalizationManager::Get(), &FTextLocalizationManager::OnPakFileMounted);
}

void BeginInitTextLocalization()
{
	LLM_SCOPE(ELLMTag::Localization);

	SCOPED_BOOT_TIMING("BeginInitTextLocalization");
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("BeginInitTextLocalization"), STAT_BeginInitTextLocalization, STATGROUP_LoadTime);

	// Initialize FInternationalization before we bind to OnCultureChanged, otherwise we can accidentally initialize
	// twice since FInternationalization::Initialize sets the culture.
	FInternationalization::Get();
	FInternationalization::Get().OnCultureChanged().AddRaw(&FTextLocalizationManager::Get(), &FTextLocalizationManager::OnCultureChanged);
}

void InitEngineTextLocalization()
{
	LLM_SCOPE(ELLMTag::Localization);
	UE_SCOPED_ENGINE_ACTIVITY(TEXT("Initializing Localization"));
	SCOPED_BOOT_TIMING("InitEngineTextLocalization");
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("InitEngineTextLocalization"), STAT_InitEngineTextLocalization, STATGROUP_LoadTime);

	// Make sure any async tasks have finished before we proceed, as this init function may update non-thread safe data
	FTextLocalizationManager::Get().WaitForAsyncTasks();

	// Make sure the String Table Registry is initialized as it may trigger module loads
	FStringTableRegistry::Get();
	FStringTableRedirects::InitStringTableRedirects();

	// Run this now that the config system is definitely initialized
	// to refresh anything that was cached before it was ready
	FInternationalization::Get().RefreshCultureDisplayNames(FInternationalization::Get().GetCurrentLanguage()->GetPrioritizedParentCultureNames());

	ELocalizationLoadFlags LocLoadFlags = ELocalizationLoadFlags::None;
	LocLoadFlags |= (WITH_EDITOR ? ELocalizationLoadFlags::Editor : ELocalizationLoadFlags::None);
	LocLoadFlags |= ELocalizationLoadFlags::Engine;
	LocLoadFlags |= TextLocalizationManager::AlwaysLoadNativeLocalizationDataDuringInitialization ? ELocalizationLoadFlags::Native : ELocalizationLoadFlags::None;
	LocLoadFlags |= ELocalizationLoadFlags::Additional;
	
	ELocalizationLoadFlags ApplyLocLoadFlags = LocLoadFlags;
	ApplyLocLoadFlags |= FApp::IsGame() ? ELocalizationLoadFlags::Game : ELocalizationLoadFlags::None;

	// Setting InitializedFlags to None ensures we don't pick up the culture change
	// notification if ApplyDefaultCultureSettings changes the default culture
	{
		ETextLocalizationManagerInitializedFlags OldFlags = FTextLocalizationManager::Get().InitializedFlags.exchange(ETextLocalizationManagerInitializedFlags::None);
		ApplyDefaultCultureSettings(ApplyLocLoadFlags);
		FTextLocalizationManager::Get().InitializedFlags = OldFlags;
	}

#if WITH_EDITOR
	FTextLocalizationManager::Get().GameLocalizationPreviewAutoEnableCount = 0;
	FTextLocalizationManager::Get().bIsGameLocalizationPreviewEnabled = false;
	FTextLocalizationManager::Get().bIsLocalizationLocked = IsLocalizationLockedByConfig();
#endif

	// Clear the native cultures for the engine and editor (they will re-cache later if used)
	TextLocalizationResourceUtil::ClearNativeEngineCultureName();
#if WITH_EDITOR
	TextLocalizationResourceUtil::ClearNativeEditorCultureName();
#endif

	FTextLocalizationManager::Get().CacheLocalizedLanguageName(LocLoadFlags);

	FTextLocalizationManager::Get().QueueAsyncTask([LocLoadFlags, AvailableTextSources = FTextLocalizationManager::Get().LocalizedTextSources]()
	{
		FTextLocalizationManager::Get().LoadLocalizationResourcesForCulture_Sync(AvailableTextSources, FInternationalization::Get().GetCurrentLanguage()->GetName(), LocLoadFlags);
		FTextLocalizationManager::Get().InitializedFlags = FTextLocalizationManager::Get().InitializedFlags.load() | ETextLocalizationManagerInitializedFlags::Engine;
	});
}

void InitGameTextLocalization()
{
	if (!FApp::IsGame())
	{
		// early out because we are not a game ;)
		return;
	}

	LLM_SCOPE(ELLMTag::Localization);
	SCOPED_BOOT_TIMING("InitGameTextLocalization");
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("InitGameTextLocalization"), STAT_InitGameTextLocalization, STATGROUP_LoadTime);

	// Make sure any async tasks have finished before we proceed, as this init function may update non-thread safe data
	FTextLocalizationManager::Get().WaitForAsyncTasks();

	// Refresh the cached config data before applying the default culture, as the game may have patched in new config data since the cache was built
	FInternationalization::Get().RefreshCachedConfigData();

	// Setting InitializedFlags to None ensures we don't pick up the culture change
	// notification if ApplyDefaultCultureSettings changes the default culture
	const FString PreviousLanguage = FInternationalization::Get().GetCurrentLanguage()->GetName();
	{
		ETextLocalizationManagerInitializedFlags OldFlags = FTextLocalizationManager::Get().InitializedFlags.exchange(ETextLocalizationManagerInitializedFlags::None);
		ApplyDefaultCultureSettings(ELocalizationLoadFlags::Game);
		FTextLocalizationManager::Get().InitializedFlags = OldFlags;
	}
	const FString CurrentLanguage = FInternationalization::Get().GetCurrentLanguage()->GetName();

	// Clear the native cultures for the game (it will re-cache later if used)
	TextLocalizationResourceUtil::ClearNativeProjectCultureName();

	ELocalizationLoadFlags LocLoadFlags = ELocalizationLoadFlags::Game;
	LocLoadFlags |= TextLocalizationManager::AlwaysLoadNativeLocalizationDataDuringInitialization ? ELocalizationLoadFlags::Native : ELocalizationLoadFlags::None;
	if (PreviousLanguage != CurrentLanguage)
	{
		// If the active language changed, then we also need to reload the Engine and Additional localization data 
		// too, as this wouldn't have happened when the culture changed above due to the InitializedFlags guard
		LocLoadFlags |= ELocalizationLoadFlags::Engine;
		LocLoadFlags |= ELocalizationLoadFlags::Additional;
	}

	FTextLocalizationManager::Get().CacheLocalizedLanguageName(LocLoadFlags);

	FTextLocalizationManager::Get().QueueAsyncTask([LocLoadFlags, AvailableTextSources = FTextLocalizationManager::Get().LocalizedTextSources]()
	{
		FTextLocalizationManager::Get().LoadLocalizationResourcesForCulture_Sync(AvailableTextSources, FInternationalization::Get().GetCurrentLanguage()->GetName(), LocLoadFlags);
		FTextLocalizationManager::Get().InitializedFlags = FTextLocalizationManager::Get().InitializedFlags.load() | ETextLocalizationManagerInitializedFlags::Game;

		//FTextLocalizationManager::Get().DumpMemoryInfo();
		//Worse when Compacting because we remove growth space and force new growth space to be reallocated the next time an entry is added which is going to be bigger than what we removed anyway...
		//FTextLocalizationManager::Get().CompactDataStructures();
		//FTextLocalizationManager::Get().DumpMemoryInfo();
	});
}

FTextLocalizationManager::FDisplayStringsForLocalizationTarget& FTextLocalizationManager::FDisplayStringsByLocalizationTargetId::FindOrAdd(FStringView InLocalizationTargetPath, int32* OutLocalizationTargetPathId)
{
	LLM_SCOPE_BYNAME(TEXT("Localization/DisplayStringsByTarget"));

	check(!InLocalizationTargetPath.IsEmpty());

	FString NormalizedLocalizationTargetPath = FPaths::ConvertRelativePathToFull(FString(InLocalizationTargetPath));
	FPaths::NormalizeDirectoryName(NormalizedLocalizationTargetPath);

	int32 LocalizationTargetPathId = LocalizationTargetPathsToIds.FindRef(NormalizedLocalizationTargetPath, INDEX_NONE);
	if (LocalizationTargetPathId == INDEX_NONE)
	{
		LocalizationTargetPathId = LocalizationTargets.Emplace(FDisplayStringsForLocalizationTarget{ MoveTemp(NormalizedLocalizationTargetPath) });
		LocalizationTargetPathsToIds.Add(LocalizationTargets[LocalizationTargetPathId].LocalizationTargetPath, LocalizationTargetPathId);
	}

	if (OutLocalizationTargetPathId)
	{
		*OutLocalizationTargetPathId = LocalizationTargetPathId;
	}
	return LocalizationTargets[LocalizationTargetPathId];
}

FTextLocalizationManager::FDisplayStringsForLocalizationTarget* FTextLocalizationManager::FDisplayStringsByLocalizationTargetId::Find(const int32 InLocalizationTargetPathId)
{
	return LocalizationTargets.IsValidIndex(InLocalizationTargetPathId)
		? &LocalizationTargets[InLocalizationTargetPathId]
		: nullptr;
}

void FTextLocalizationManager::FDisplayStringsByLocalizationTargetId::TrackTextId(const int32 InCurrentLocalizationPathId, const int32 InNewLocalizationPathId, const FTextId& InTextId)
{
	if (InCurrentLocalizationPathId == InNewLocalizationPathId)
	{
		return;
	}

	LLM_SCOPE_BYNAME(TEXT("Localization/DisplayStringsByTarget"));

	if (FDisplayStringsForLocalizationTarget* DisplayStringsForCurrentLocalizationTarget = Find(InCurrentLocalizationPathId);
		DisplayStringsForCurrentLocalizationTarget && DisplayStringsForCurrentLocalizationTarget->bIsMounted)
	{
		DisplayStringsForCurrentLocalizationTarget->TextIds.Remove(InTextId);
	}

	if (FDisplayStringsForLocalizationTarget* DisplayStringsForNewLocalizationTarget = Find(InNewLocalizationPathId);
		DisplayStringsForNewLocalizationTarget && DisplayStringsForNewLocalizationTarget->bIsMounted)
	{
		DisplayStringsForNewLocalizationTarget->TextIds.Add(InTextId);
	}
}

FTextLocalizationManager& FTextLocalizationManager::Get()
{
	return TLazySingleton<FTextLocalizationManager>::Get();
}

void FTextLocalizationManager::TearDown()
{
	FTextLocalizationManager::Get().WaitForAsyncTasks();

	FTextCache::TearDown();
	TLazySingleton<FTextLocalizationManager>::TearDown();
	FTextKey::TearDown();
}

bool FTextLocalizationManager::IsDisplayStringSupportEnabled()
{
	switch (static_cast<TextLocalizationManager::EDisplayStringSupport>(TextLocalizationManager::DisplayStringSupport))
	{
	case TextLocalizationManager::EDisplayStringSupport::Auto:
#if UE_EDITOR
		return true; // IsRunningDedicatedServer asserts during static-init if called in the editor
#else
		return !IsRunningDedicatedServer();
#endif
	case TextLocalizationManager::EDisplayStringSupport::Enabled:
		return true;
	case TextLocalizationManager::EDisplayStringSupport::Disabled:
		return false;
	default:
		checkf(false, TEXT("Unknown EDisplayStringSupport!"));
		break;
	}
	return true;
}

FTextLocalizationManager::FTextLocalizationManager()
	: TextRevisionCounter(1) // Default to 1 as 0 is considered unset
	, LocResTextSource(MakeShared<FLocalizationResourceTextSource>())
	, PolyglotTextSource(MakeShared<FPolyglotTextSource>())
{
	const bool bRefreshResources = false;
	RegisterTextSource(LocResTextSource.ToSharedRef(), bRefreshResources);
	RegisterTextSource(PolyglotTextSource.ToSharedRef(), bRefreshResources);
}

FTextLocalizationManager::~FTextLocalizationManager()
{
	// Note: Explicit destructor needed since AsyncLocalizationTask is using a forward declared type in the header
}

void FTextLocalizationManager::DumpMemoryInfo() const
{
	{
		FReadScopeLock ScopeLock(DisplayStringTableRW);
		UE_LOG(LogTextLocalizationManager, Log, TEXT("DisplayStringLookupTable.GetAllocatedSize()=%" SIZE_T_FMT " elems=%d"), DisplayStringLookupTable.GetAllocatedSize(), DisplayStringLookupTable.Num());
	}
	{
		FReadScopeLock ScopeLock(TextRevisionRW);
		UE_LOG(LogTextLocalizationManager, Log, TEXT("LocalTextRevisions.GetAllocatedSize()=%" SIZE_T_FMT " elems=%d"), LocalTextRevisions.GetAllocatedSize(), LocalTextRevisions.Num());
	}
}

void FTextLocalizationManager::CompactDataStructures()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FTextLocalizationManager::CompactDataStructures);
	LLM_SCOPE_BYNAME(TEXT("Localization/DisplayStrings"));

	double StartTime = FPlatformTime::Seconds();
	{
		FWriteScopeLock ScopeLock(DisplayStringTableRW);
		DisplayStringLookupTable.Shrink();
	}
	{
		FWriteScopeLock ScopeLock(TextRevisionRW);
		LocalTextRevisions.Shrink();
	}
	FTextKey::CompactDataStructures();
	UE_LOGF(LogTextLocalizationManager, Log, "Compacting localization data took %6.2fms", 1000.0 * (FPlatformTime::Seconds() - StartTime));
}

#if ENABLE_LOC_TESTING
void FTextLocalizationManager::DumpLiveTableImpl(const FString* NamespaceFilter, const FString* KeyFilter, const FString* DisplayStringFilter, TFunctionRef<void(const FTextId& Id, const FTextConstDisplayStringRef& DisplayString)> Callback) const
{
	FSlowHeartBeatScope SuspendHeartBeat;

	FDisplayStringLookupTable DisplayStringLookupTableToDump;
	{
		auto PassesFilter = [](const FString& Str, const FString* Filter)
		{
			return !Filter || Str.MatchesWildcard(*Filter, ESearchCase::IgnoreCase); // Note: This is case insensitive since its used from a debug command
		};

		FReadScopeLock ScopeLock(DisplayStringTableRW);
		DisplayStringLookupTableToDump.Reserve(DisplayStringLookupTable.Num());
		for (const auto& DisplayStringPair : DisplayStringLookupTable)
		{
			if (PassesFilter(DisplayStringPair.Key.GetNamespace().ToString(), NamespaceFilter) &&
				PassesFilter(DisplayStringPair.Key.GetKey().ToString(), KeyFilter) &&
				PassesFilter(**DisplayStringPair.Value.DisplayString, DisplayStringFilter))
			{
				DisplayStringLookupTableToDump.Add(DisplayStringPair.Key, DisplayStringPair.Value);
			}
		}
	}

	DisplayStringLookupTableToDump.KeySort([](const FTextId& A, const FTextId& B)
	{
		const int32 NamespaceResult = FCString::Strcmp(*A.GetNamespace().ToString(), *B.GetNamespace().ToString());
		if (NamespaceResult != 0)
		{
			return NamespaceResult < 0;
		}
		return FCString::Strcmp(*A.GetKey().ToString(), *B.GetKey().ToString()) < 0;
	});

	for (const auto& DisplayStringPair : DisplayStringLookupTableToDump)
	{
		Callback(DisplayStringPair.Key, DisplayStringPair.Value.DisplayString);
	}
}

void FTextLocalizationManager::DumpLiveTable(const FString* NamespaceFilter, const FString* KeyFilter, const FString* DisplayStringFilter, const FLogCategoryBase* CategoryOverride) const
{
#if !NO_LOGGING
	const FLogCategoryBase& Category = CategoryOverride ? *CategoryOverride : LogLocalization;

	UE_LOG_REF(Category, Display, TEXT("----------------------------------------------------------------------"));

	DumpLiveTableImpl(NamespaceFilter, KeyFilter, DisplayStringFilter, [&Category](const FTextId& Id, const FTextConstDisplayStringRef& DisplayString)
	{
		UE_LOG_REF(Category, Display, TEXT("LiveTableEntry: Namespace: '%s', Key: '%s', DisplayString: '%s'"), *Id.GetNamespace().ToString(), *Id.GetKey().ToString(), *TextLocalizationManager::ConditionDisplayStringForLog(*DisplayString));
	});

	UE_LOG_REF(Category, Display, TEXT("----------------------------------------------------------------------"));
#endif // !NO_LOGGING
}

void FTextLocalizationManager::DumpLiveTable(const FString& OutputFilename, const FString* NamespaceFilter, const FString* KeyFilter, const FString* DisplayStringFilter) const
{
	FString DumpString;

	DumpLiveTableImpl(NamespaceFilter, KeyFilter, DisplayStringFilter, [&DumpString](const FTextId& Id, const FTextConstDisplayStringRef& DisplayString)
	{
		DumpString += FString::Printf(TEXT("LiveTableEntry: Namespace: '%s', Key: '%s', DisplayString: '%s'"), *Id.GetNamespace().ToString(), *Id.GetKey().ToString(), *TextLocalizationManager::ConditionDisplayStringForLog(*DisplayString));
		DumpString += LINE_TERMINATOR;
	});

	FFileHelper::SaveStringToFile(DumpString, *OutputFilename, FFileHelper::EEncodingOptions::ForceUTF8WithoutBOM);
}

void FTextLocalizationManager::AddOrUpdateDisplayStringInLiveTable(const FString& Namespace, const FString& Key, const FString& DisplayString, const FString* const SourceStringPtr)
{
	const FTextId TextId(Namespace, Key);

	// Lock while updating the table
	FWriteScopeLock ScopeLock(DisplayStringTableRW);

	if (FDisplayStringEntry* LiveEntry = DisplayStringLookupTable.Find(TextId))
	{
		if (SourceStringPtr)
		{
			LiveEntry->SourceStringHash = FTextLocalizationResource::HashString(*SourceStringPtr);
		}
		LiveEntry->DisplayString = MakeTextDisplayString(CopyTemp(DisplayString));
		DirtyLocalRevisionForTextId(TextId);

		UE_LOGF(LogConsoleResponse, Display, "Updated string for Namespace='%ls', Key='%ls' to DisplayString='%ls'", *Namespace, *Key, *TextLocalizationManager::ConditionDisplayStringForLog(DisplayString));
	}
	else if (SourceStringPtr)
	{
		{
			// Add new entry
			FDisplayStringEntry NewLiveEntry(
				FTextKey(),													/*LocResID*/
				INDEX_NONE,													/*LocalizationTargetPathId*/
				FTextLocalizationResource::HashString(*SourceStringPtr),	/*SourceStringHash*/
				MakeTextDisplayString(CopyTemp(DisplayString))				/*String*/
			);

			DisplayStringLookupTable.Add(TextId, MoveTemp(NewLiveEntry));
		}

		UE_LOGF(LogConsoleResponse, Display, "Added string for Namespace='%ls', Key='%ls' to DisplayString='%ls'", *Namespace, *Key, *TextLocalizationManager::ConditionDisplayStringForLog(DisplayString));
	}
	else
	{
		UE_LOGF(LogConsoleResponse, Display, "String not found for Namespace='%ls', Key='%ls'", *Namespace, *Key);
	}
}

#endif

FString FTextLocalizationManager::GetRequestedLanguageName() const
{
	ERequestedCultureOverrideLevel LanguageOverrideLevel = ERequestedCultureOverrideLevel::Defaults;
	return GetRequestedLanguage(LanguageOverrideLevel);
}

FString FTextLocalizationManager::GetRequestedLocaleName() const
{
	ERequestedCultureOverrideLevel LocaleOverrideLevel = ERequestedCultureOverrideLevel::Defaults;
	return GetRequestedLocale(LocaleOverrideLevel);
}

FString FTextLocalizationManager::GetNativeCultureName(const ELocalizedTextSourceCategory InCategory) const
{
	FString NativeCultureName;
	for (const TSharedPtr<ILocalizedTextSource>& LocalizedTextSource : LocalizedTextSources)
	{
		if (LocalizedTextSource->GetNativeCultureName(InCategory, NativeCultureName))
		{
			break;
		}
	}
	return NativeCultureName;
}

TArray<FString> FTextLocalizationManager::GetLocalizedCultureNames(const ELocalizationLoadFlags InLoadFlags) const
{
	TSet<FString> LocalizedCultureNameSet;
	for (const TSharedPtr<ILocalizedTextSource>& LocalizedTextSource : LocalizedTextSources)
	{
		LocalizedTextSource->GetLocalizedCultureNames(InLoadFlags, LocalizedCultureNameSet);
	}

	TArray<FString> LocalizedCultureNames = LocalizedCultureNameSet.Array();
	LocalizedCultureNames.Sort();

	// Remove any cultures that were explicitly disallowed
	FInternationalization& I18N = FInternationalization::Get();
	LocalizedCultureNames.RemoveAll([&I18N](const FString& InCultureName)
	{
		return !I18N.IsCultureAllowed(InCultureName);
	});
	
	return LocalizedCultureNames;
}

FString FTextLocalizationManager::GetLocalizedLanguageName() const
{
	return LocalizedLanguageName;
}

int32 FTextLocalizationManager::GetLocalizationTargetPathId(FStringView InLocalizationTargetPath)
{
	FWriteScopeLock ScopeLock(DisplayStringTableRW);

	int32 LocalizationTargetPathId = INDEX_NONE;
	DisplayStringsByLocalizationTargetId.FindOrAdd(InLocalizationTargetPath, &LocalizationTargetPathId);
	return LocalizationTargetPathId;
}

void FTextLocalizationManager::RegisterTextSource(const TSharedRef<ILocalizedTextSource>& InLocalizedTextSource, const bool InRefreshResources)
{
	LocalizedTextSources.Add(InLocalizedTextSource);
	LocalizedTextSources.StableSort([](const TSharedPtr<ILocalizedTextSource>& InLocalizedTextSourceOne, const TSharedPtr<ILocalizedTextSource>& InLocalizedTextSourceTwo)
	{
		return InLocalizedTextSourceOne->GetPriority() > InLocalizedTextSourceTwo->GetPriority();
	});

	if (InRefreshResources)
	{
		RefreshResources();
	}
}

void FTextLocalizationManager::RegisterPolyglotTextData(const FPolyglotTextData& InPolyglotTextData, const bool InAddDisplayString)
{
	check(PolyglotTextSource.IsValid());
	RegisterPolyglotTextData(TArrayView<const FPolyglotTextData>(&InPolyglotTextData, 1), InAddDisplayString);
}

void FTextLocalizationManager::RegisterPolyglotTextData(TArrayView<const FPolyglotTextData> InPolyglotTextDataArray, const bool InAddDisplayStrings)
{
	for (const FPolyglotTextData& PolyglotTextData : InPolyglotTextDataArray)
	{
		if (PolyglotTextData.IsValid())
		{
			PolyglotTextSource->RegisterPolyglotTextData(PolyglotTextData);
		}
	}

	if (InAddDisplayStrings)
	{
		auto GetLocalizedStringForPolyglotData = [this](const FPolyglotTextData& PolyglotTextData, FString& OutLocalizedString) -> bool
		{
			// Work out which culture to use - this is typically the current language unless we're in the 
			// editor where the game localization preview affects the language we use for game text
			FString CultureName;
#if WITH_EDITOR
			if (PolyglotTextData.GetCategory() == ELocalizedTextSourceCategory::Game)
			{
				if (bIsGameLocalizationPreviewEnabled)
				{
					CultureName = GetConfiguredGameLocalizationPreviewLanguage();
				}
				else if (!GIsEditor || ShouldForceLoadGameLocalization())
				{
					CultureName = FInternationalization::Get().GetCurrentLanguage()->GetName();
				}
			}
			else
#endif
			{
				CultureName = FInternationalization::Get().GetCurrentLanguage()->GetName();
			}

			if (!CultureName.IsEmpty())
			{
				const TArray<FString> PrioritizedCultureNames = FInternationalization::Get().GetPrioritizedCultureNames(CultureName);
				for (const FString& PrioritizedCultureName : PrioritizedCultureNames)
				{
					if (PolyglotTextData.GetLocalizedString(PrioritizedCultureName, OutLocalizedString))
					{
						return true;
					}
				}
			}

			if (PolyglotTextData.IsMinimalPatch())
			{
				return false;
			}

			OutLocalizedString = PolyglotTextData.GetNativeString();
			return true;
		};

		FTextLocalizationResource TextLocalizationResource;
		for (const FPolyglotTextData& PolyglotTextData : InPolyglotTextDataArray)
		{
			if (!PolyglotTextData.IsValid())
			{
				continue;
			}

			FString LocalizedString;
			if (GetLocalizedStringForPolyglotData(PolyglotTextData, LocalizedString))
			{
				TextLocalizationResource.AddEntry(
					PolyglotTextData.GetNamespace(),
					PolyglotTextData.GetKey(),
					PolyglotTextData.GetNativeString(),
					LocalizedString,
					0
					);
			}
		}

		if (!TextLocalizationResource.IsEmpty())
		{
			UpdateLiveTable(MoveTemp(TextLocalizationResource));
		}
	}
}

FTextConstDisplayStringPtr FTextLocalizationManager::FindDisplayString_Internal(const FTextId& TextId, const FString& SourceString) const
{
	if (!IsInitialized())
	{
		return nullptr;
	}

#if ENABLE_LOC_TESTING
	{
		FInternationalization& I18N = FInternationalization::Get();
		if (I18N.GetCurrentLanguage()->GetName() == FLeetCulture::StaticGetName())
		{
			FTextDisplayStringRef TmpDisplayString = MakeTextDisplayString(CopyTemp(SourceString));
			FInternationalization::Leetify(*TmpDisplayString);
			return TmpDisplayString;
		}
		if (I18N.GetCurrentLanguage()->GetName() == FKeysCulture::StaticGetName())
		{
			return MakeTextDisplayString(TextLocalizationManager::KeyifyTextId(TextId));
		}
	}
#endif	// ENABLE_LOC_TESTING

	FReadScopeLock ScopeLock(DisplayStringTableRW);

	if (const FDisplayStringEntry* LiveEntry = DisplayStringLookupTable.Find(TextId))
	{
		if (SourceString.IsEmpty() || LiveEntry->SourceStringHash == FTextLocalizationResource::HashString(SourceString))
		{
			return LiveEntry->DisplayString;
		}
	}

	return nullptr;
}

FTextConstDisplayStringPtr FTextLocalizationManager::FindDisplayString(const FTextKey& Namespace, const FTextKey& Key, const FString* const SourceStringPtr) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FTextLocalizationManager::FindDisplayString);

	if (Key.IsEmpty() || !FTextLocalizationManager::IsDisplayStringSupportEnabled())
	{
		return nullptr;
	}

	auto GetSourceStringRef = [SourceStringPtr]() -> const FString&
	{
		if (SourceStringPtr)
		{
			return *SourceStringPtr;
		}

		static const FString EmptyString;
		return EmptyString;
	};

	return FindDisplayString_Internal(FTextId(Namespace, Key), GetSourceStringRef());
}

FTextConstDisplayStringPtr FTextLocalizationManager::GetDisplayString(const FTextKey& Namespace, const FTextKey& Key, const FString* const SourceStringPtr) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FTextLocalizationManager::GetDisplayString);

	if (Key.IsEmpty() || !FTextLocalizationManager::IsDisplayStringSupportEnabled())
	{
		return nullptr;
	}

	FTextId TextId(Namespace, Key);

	// In builds with stable keys enabled, we want to find the display string for the "clean" version of the text (if the sources match) as this is the only version that is translated
#if USE_STABLE_LOCALIZATION_KEYS
	{
		const FString FullNamespace = TextId.GetNamespace().ToString();
		const FString DisplayNamespace = TextNamespaceUtil::StripPackageNamespace(FullNamespace);
		if (!DisplayNamespace.Equals(FullNamespace, ESearchCase::CaseSensitive))
		{
			TextId = FTextId(DisplayNamespace, TextId.GetKey());
		}
	}
#endif	// USE_STABLE_LOCALIZATION_KEYS

	auto GetSourceStringRef = [SourceStringPtr]() -> const FString&
	{
		if (SourceStringPtr)
		{
			return *SourceStringPtr;
		}

		static const FString EmptyString;
		return EmptyString;
	};

	const FString& SourceString = GetSourceStringRef();

	if (FTextConstDisplayStringPtr DisplayString = FindDisplayString_Internal(TextId, SourceString))
	{
		return DisplayString;
	}

#if ENABLE_LOC_TESTING
	if (FCommandLine::IsInitialized())
	{
		static const bool bShouldLEETIFYUnlocalizedString = FParse::Param(FCommandLine::Get(), TEXT("LEETIFYUnlocalized"));
		if (bShouldLEETIFYUnlocalizedString)
		{
			FTextDisplayStringRef TmpDisplayString = MakeTextDisplayString(CopyTemp(SourceString));
			FInternationalization::Leetify(*TmpDisplayString);
			return TmpDisplayString;
		}
	}
#endif

	return nullptr;
}

#if WITH_EDITORONLY_DATA
bool FTextLocalizationManager::GetLocResID(const FTextKey& Namespace, const FTextKey& Key, FString& OutLocResId) const
{
	FReadScopeLock ScopeLock(DisplayStringTableRW);

	const FTextId TextId(Namespace, Key);

	const FDisplayStringEntry* LiveEntry = DisplayStringLookupTable.Find(TextId);

	if (LiveEntry != nullptr && !LiveEntry->LocResID.IsEmpty())
	{
		LiveEntry->LocResID.ToString(OutLocResId);
		return true;
	}

	return false;
}
#endif

UE_AUTORTFM_ALWAYS_OPEN
uint16 FTextLocalizationManager::GetTextRevision() const
{
	FReadScopeLock ScopeLock(TextRevisionRW);
	return TextRevisionCounter;
}

uint16 FTextLocalizationManager::GetLocalRevisionForTextId(const FTextId& InTextId) const
{
	if (!InTextId.IsEmpty())
	{
		FReadScopeLock ScopeLock(TextRevisionRW);
		if (const uint16* FoundLocalRevision = LocalTextRevisions.Find(InTextId))
		{
			return *FoundLocalRevision;
		}
	}
	return 0;
}

UE_AUTORTFM_ALWAYS_OPEN
void FTextLocalizationManager::GetTextRevisions(const FTextId& InTextId, uint16& OutGlobalTextRevision, uint16& OutLocalTextRevision) const
{
	FReadScopeLock ScopeLock(TextRevisionRW);

	OutGlobalTextRevision = TextRevisionCounter;
	if (const uint16* FoundLocalRevision = InTextId.IsEmpty() ? nullptr : LocalTextRevisions.Find(InTextId))
	{
		OutLocalTextRevision = *FoundLocalRevision;
	}
	else
	{
		OutLocalTextRevision = 0;
	}
}

void FTextLocalizationManager::UpdateFromLocalizationResource(const FString& LocalizationResourceFilePath)
{
	FTextLocalizationResource TextLocalizationResource;
	TextLocalizationResource.LoadFromFile(LocalizationResourceFilePath, 0);
	UpdateLiveTable(MoveTemp(TextLocalizationResource));
}

void FTextLocalizationManager::UpdateFromLocalizationResource(const FTextLocalizationResource& TextLocalizationResource)
{
	UpdateLiveTable(CopyTemp(TextLocalizationResource));
}

void FTextLocalizationManager::WaitForAsyncTasks()
{
	SCOPED_BOOT_TIMING("FTextLocalizationManager::WaitForAsyncTasks");
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FTextLocalizationManager::WaitForAsyncTasks"), STAT_WaitForAsyncLocalizationTasks, STATGROUP_LoadTime);

	if (AsyncLocalizationTask)
	{
		if (FTaskGraphInterface::IsRunning())
		{
			FTaskGraphInterface::Get().WaitUntilTaskCompletes(AsyncLocalizationTask);
		}
		AsyncLocalizationTask.SafeRelease();
	}
}

void FTextLocalizationManager::NotifyWhenAsyncTasksCompleted(TUniqueFunction<void()>&& Notification)
{
	if (AsyncLocalizationTask)
	{
		QueueAsyncTask(MoveTemp(Notification));
	}
	else
	{
		Notification();
	}
}

void FTextLocalizationManager::RefreshResources(TUniqueFunction<void()>&& Notification)
{
	ELocalizationLoadFlags LocLoadFlags = ELocalizationLoadFlags::None;
	LocLoadFlags |= (WITH_EDITOR ? ELocalizationLoadFlags::Editor : ELocalizationLoadFlags::None);
	LocLoadFlags |= (FApp::IsGame() ? ELocalizationLoadFlags::Game : ELocalizationLoadFlags::None);
	LocLoadFlags |= ELocalizationLoadFlags::Engine;
	LocLoadFlags |= ELocalizationLoadFlags::Native;
	LocLoadFlags |= ELocalizationLoadFlags::Additional;

	CacheLocalizedLanguageName(LocLoadFlags);

	LoadLocalizationResourcesForCulture_Async(FInternationalization::Get().GetCurrentLanguage()->GetName(), LocLoadFlags, MoveTemp(Notification));
}

void FTextLocalizationManager::HandleLocalizationTargetsMounted(TArrayView<const FString> LocalizationTargetPaths, TUniqueFunction<void()>&& Notification)
{
	if (!IsInitialized() || LocalizationTargetPaths.IsEmpty())
	{
		// If we've not yet loaded localization data then there's nothing to do
		if (Notification)
		{
			Notification();
		}
		return;
	}

	// Nothing to do?
	if (!FTextLocalizationManager::IsDisplayStringSupportEnabled())
	{
		if (Notification)
		{
			Notification();
		}
		return;
	}

	// Mark the targets as mounted before loading any of their data
	{
		FWriteScopeLock ScopeLock(DisplayStringTableRW);
		for (const FString& LocalizationTargetPath : LocalizationTargetPaths)
		{
			FDisplayStringsForLocalizationTarget& DisplayStringsForLocalizationTarget = DisplayStringsByLocalizationTargetId.FindOrAdd(LocalizationTargetPath);
			DisplayStringsForLocalizationTarget.bIsMounted = true;
		}
	}

	ELocalizationLoadFlags LocLoadFlags = ELocalizationLoadFlags::None;
	LocLoadFlags |= (WITH_EDITOR ? ELocalizationLoadFlags::Editor : ELocalizationLoadFlags::None);
	LocLoadFlags |= (FApp::IsGame() ? ELocalizationLoadFlags::Game : ELocalizationLoadFlags::None);
	LocLoadFlags |= ELocalizationLoadFlags::Engine;
	LocLoadFlags |= ELocalizationLoadFlags::Native;

	const TArray<FString> PrioritizedCultureNames = FInternationalization::Get().GetPrioritizedCultureNames(FInternationalization::Get().GetCurrentLanguage()->GetName());

	QueueAsyncTask([AvailableTextSources = LocalizedTextSources, LocalizationTargetPaths = TArray<FString>(LocalizationTargetPaths), PrioritizedCultureNames = TArray<FString>(PrioritizedCultureNames), LocLoadFlags, Notification = MoveTemp(Notification)]()
	{
		for (const FString& LocalizationTargetPath : LocalizationTargetPaths)
		{
			if (!IFileManager::Get().DirectoryExists(*LocalizationTargetPath))
			{
				continue;
			}

			FTextLocalizationMetaDataResource LocMetaResource;
			{
				const FString LocMetaFilename = FPaths::GetBaseFilename(LocalizationTargetPath) + TEXT(".locmeta");
				LocMetaResource.LoadFromFile(LocalizationTargetPath / LocMetaFilename);
			}

			// We don't allow dynamically loaded UGC localization data to replace any existing data
			const ELocalizationLoadFlags FinalLocLoadFlags = LocLoadFlags | (LocMetaResource.bIsUGC ? ELocalizationLoadFlags::SkipExisting : ELocalizationLoadFlags::None);
			FTextLocalizationManager::Get().LoadLocalizationTargetsForPrioritizedCultures_Sync(AvailableTextSources, MakeArrayView(&LocalizationTargetPath, 1), PrioritizedCultureNames, FinalLocLoadFlags);
		}
		if (Notification)
		{
			Notification();
		}
	});
}

void FTextLocalizationManager::HandleLocalizationTargetsUnmounted(TArrayView<const FString> LocalizationTargetPaths, TUniqueFunction<void()>&& Notification)
{
	if (!IsInitialized() || LocalizationTargetPaths.IsEmpty())
	{
		// If we've not yet loaded localization data then there's nothing to do
		if (Notification)
		{
			Notification();
		}
		return;
	}
	
	// Nothing to do?
	if (!FTextLocalizationManager::IsDisplayStringSupportEnabled())
	{
		if (Notification)
		{
			Notification();
		}
		return;
	}

	// Async update the live table
	QueueAsyncTask([LocalizationTargetPaths = TArray<FString>(LocalizationTargetPaths), Notification = MoveTemp(Notification)]()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FTextLocalizationManager::HandleLocalizationTargetsUnmounted);
		LLM_SCOPE_BYNAME(TEXT("Localization/DisplayStrings"));

		FTextLocalizationManager& TLM = FTextLocalizationManager::Get();
		FTextCache& TextCache = FTextCache::Get();

		// Lock while updating the tables
		FWriteScopeLock ScopeLock(TLM.DisplayStringTableRW);

		// Discard the data for each localization target that was unmounted, and mark the target as no longer mounted so that we no longer track its text IDs
		for (const FString& LocalizationTargetPath : LocalizationTargetPaths)
		{
			FDisplayStringsForLocalizationTarget& DisplayStringsForLocalizationTarget = TLM.DisplayStringsByLocalizationTargetId.FindOrAdd(LocalizationTargetPath);
			if (DisplayStringsForLocalizationTarget.bIsMounted)
			{
				for (const FTextId& TextId : DisplayStringsForLocalizationTarget.TextIds)
				{
					TLM.DisplayStringLookupTable.Remove(TextId);
				}
				TextCache.RemoveCache(DisplayStringsForLocalizationTarget.TextIds);

				DisplayStringsForLocalizationTarget.TextIds.Empty();
				DisplayStringsForLocalizationTarget.bIsMounted = false;
			}
		}

		// Allow any lingering texts that were referencing the unloaded display strings to release their references
		TLM.DirtyTextRevision();

		if (Notification)
		{
			Notification();
		}
	});
}

void FTextLocalizationManager::OnPakFileMounted(const IPakFile& PakFile)
{
	SCOPED_BOOT_TIMING("FTextLocalizationManager::OnPakFileMounted");
	LLM_SCOPE(ELLMTag::Localization);

	int32 ChunkId = PakFile.PakGetPakchunkIndex();
	if (ChunkId == INDEX_NONE || ChunkId == 0 || PakFile.GetNumFiles() == 0)
	{
		// Skip empty (IoStore), non-chunked PAK files, and chunk 0 as that contains the standard localization data
		return;
	}

	LoadChunkedLocalizationResources_Async(ChunkId, PakFile.PakGetPakFilename());
}

void FTextLocalizationManager::OnCultureChanged()
{
    if (!IsInitialized())
	{
		// Ignore culture changes while the text localization manager is still being initialized
		// The correct data will be loaded by EndInitTextLocalization
		return;
	}

	if (!FTextLocalizationManager::IsDisplayStringSupportEnabled())
	{
		// When display strings are disabled just bump the text revision (so that generated text updates correctly for the new locale) and bail
		DirtyTextRevision();
		return;
	}

#if ENABLE_LOC_TESTING
	if (FInternationalization::Get().GetCurrentLanguage()->GetName() == FLeetCulture::StaticGetName() || FInternationalization::Get().GetCurrentLanguage()->GetName() == FKeysCulture::StaticGetName())
	{
		// When switching to a debug culture, just bump the text revision (so that the dynamically generated debug display strings are rebuilt) and bail
		DirtyTextRevision();
		return;
	}
#endif	// ENABLE_LOC_TESTING

	RefreshResources();

	if (!TextLocalizationManager::AsyncLoadLocalizationDataOnLanguageChange)
	{
		FSlowHeartBeatScope SuspendHeartBeat;
		WaitForAsyncTasks();
	}
}

void FTextLocalizationManager::LoadLocalizationResourcesForCulture_Sync(TArrayView<const TSharedPtr<ILocalizedTextSource>> AvailableTextSources, const FString& CultureName, const ELocalizationLoadFlags LocLoadFlags)
{
    LLM_SCOPE(ELLMTag::Localization);

	// Don't attempt to process an empty culture name, early-out.
	if (CultureName.IsEmpty())
	{
		return;
	}

	// Can't load localization resources for a culture that doesn't exist, early-out.
	const FCulturePtr Culture = FInternationalization::Get().GetCulture(CultureName);
	if (!Culture.IsValid())
	{
		return;
	}

	LoadLocalizationResourcesForPrioritizedCultures_Sync(AvailableTextSources, FInternationalization::Get().GetPrioritizedCultureNames(CultureName), LocLoadFlags);
}

void FTextLocalizationManager::LoadLocalizationResourcesForCulture_Async(const FString& CultureName, const ELocalizationLoadFlags LocLoadFlags, TUniqueFunction<void()>&& Notification)
{
	QueueAsyncTask([AvailableTextSources = LocalizedTextSources, CultureName, LocLoadFlags, Notification = MoveTemp(Notification)]()
	{
		FTextLocalizationManager::Get().LoadLocalizationResourcesForCulture_Sync(AvailableTextSources, CultureName, LocLoadFlags);
		if (Notification)
		{
			Notification();
		}
	});
}

void FTextLocalizationManager::LoadLocalizationResourcesForPrioritizedCultures_Sync(TArrayView<const TSharedPtr<ILocalizedTextSource>> AvailableTextSources, TArrayView<const FString> PrioritizedCultureNames, const ELocalizationLoadFlags LocLoadFlags)
{
	LLM_SCOPE(ELLMTag::Localization);
	TRACE_CPUPROFILER_EVENT_SCOPE(FTextLocalizationManager::LoadLocalizationResourcesForPrioritizedCultures);

	// Nothing to do?
	if (!FTextLocalizationManager::IsDisplayStringSupportEnabled() || PrioritizedCultureNames.Num() == 0)
	{
		return;
	}

	const ELocalizationLoadFlags FinalLocLoadFlags = LocLoadFlags | (ShouldForceLoadGameLocalization() ? ELocalizationLoadFlags::ForceLocalizedGame : ELocalizationLoadFlags::None);

	// Load the resources from each text source
	FTextLocalizationResource TextLocalizationResource;
	for (const TSharedPtr<ILocalizedTextSource>& LocalizedTextSource : AvailableTextSources)
	{
		LLM_SCOPE_BYNAME(TEXT("Localization/DisplayStrings"));
		LocalizedTextSource->LoadLocalizedResources(FinalLocLoadFlags, PrioritizedCultureNames, TextLocalizationResource, TextLocalizationResource);
	}

	// Apply the new display string data
	{
		FUpdateLiveTableOptions UpdateOptions;
		UpdateOptions.bReplaceExisting = !EnumHasAnyFlags(FinalLocLoadFlags, ELocalizationLoadFlags::SkipExisting);

		UpdateLiveTable(MoveTemp(TextLocalizationResource), UpdateOptions);
	}
}

void FTextLocalizationManager::LoadLocalizationResourcesForPrioritizedCultures_Async(TArrayView<const FString> PrioritizedCultureNames, const ELocalizationLoadFlags LocLoadFlags, TUniqueFunction<void()>&& Notification)
{
	QueueAsyncTask([AvailableTextSources = LocalizedTextSources, PrioritizedCultureNames = TArray<FString>(PrioritizedCultureNames), LocLoadFlags, Notification = MoveTemp(Notification)]()
	{
		FTextLocalizationManager::Get().LoadLocalizationResourcesForPrioritizedCultures_Sync(AvailableTextSources, PrioritizedCultureNames, LocLoadFlags);
		if (Notification)
		{
			Notification();
		}
	});
}

void FTextLocalizationManager::LoadLocalizationTargetsForPrioritizedCultures_Sync(TArrayView<const TSharedPtr<ILocalizedTextSource>> AvailableTextSources, TArrayView<const FString> LocalizationTargetPaths, TArrayView<const FString> PrioritizedCultureNames, const ELocalizationLoadFlags LocLoadFlags)
{
	LLM_SCOPE(ELLMTag::Localization);
	TRACE_CPUPROFILER_EVENT_SCOPE(FTextLocalizationManager::LoadLocalizationTargetsForPrioritizedCultures);

	// Nothing to do?
	if (!FTextLocalizationManager::IsDisplayStringSupportEnabled() || PrioritizedCultureNames.Num() == 0 || LocalizationTargetPaths.Num() == 0)
	{
		return;
	}

	// Load the resources from each localization target
	FTextLocalizationResource TextLocalizationResource;
	for (const FString& LocalizationTargetPath : LocalizationTargetPaths)
	{
		UE_LOGF(LogTextLocalizationManager, Verbose, "Loading LocRes data from '%ls'", *LocalizationTargetPath);
	}
	{
		LLM_SCOPE_BYNAME(TEXT("Localization/DisplayStrings"));
		LocResTextSource->LoadLocalizedResourcesFromPaths(LocalizationTargetPaths, LocalizationTargetPaths, TArrayView<FString>(), LocLoadFlags, PrioritizedCultureNames, TextLocalizationResource, TextLocalizationResource);
	}

	// Allow any higher priority text sources to override the additional text loaded (eg, to allow polyglot hot-fixes to take priority)
	// Note: If any text sources don't support dynamic queries, then we must do a much slower full refresh instead :(
	bool bNeedsFullRefresh = false;
	{
		// Copy the IDs array as QueryLocalizedResource can update the map
		TArray<FTextId> NewTextIds;
		TextLocalizationResource.Entries.GenerateKeyArray(NewTextIds);

		for (const TSharedPtr<ILocalizedTextSource>& LocalizedTextSource : AvailableTextSources)
		{
			if (LocalizedTextSource->GetPriority() <= LocResTextSource->GetPriority())
			{
				continue;
			}

			LLM_SCOPE_BYNAME(TEXT("Localization/DisplayStrings"));
			for (const FTextId& NewTextId : NewTextIds)
			{
				if (LocalizedTextSource->QueryLocalizedResource(LocLoadFlags, PrioritizedCultureNames, NewTextId, TextLocalizationResource, TextLocalizationResource) == EQueryLocalizedResourceResult::NotImplemented)
				{
					bNeedsFullRefresh = true;
					break;
				}
			}

			if (bNeedsFullRefresh)
			{
				break;
			}
		}
	}

	// Apply the new data
	if (bNeedsFullRefresh)
	{
		UE_LOGF(LogTextLocalizationManager, Verbose, "Patching LocRes data failed, performing full refresh");
		LoadLocalizationResourcesForPrioritizedCultures_Sync(AvailableTextSources, PrioritizedCultureNames, LocLoadFlags);
	}
	else
	{
		FUpdateLiveTableOptions UpdateOptions;
		UpdateOptions.bReplaceExisting = !EnumHasAnyFlags(LocLoadFlags, ELocalizationLoadFlags::SkipExisting);

		UE_LOGF(LogTextLocalizationManager, Verbose, "Patching LocRes data for %d entries", TextLocalizationResource.Entries.Num());
		UpdateLiveTable(MoveTemp(TextLocalizationResource), UpdateOptions);
	}
}

void FTextLocalizationManager::LoadLocalizationTargetsForPrioritizedCultures_Async(TArrayView<const FString> LocalizationTargetPaths, TArrayView<const FString> PrioritizedCultureNames, const ELocalizationLoadFlags LocLoadFlags, TUniqueFunction<void()>&& Notification)
{
	QueueAsyncTask([AvailableTextSources = LocalizedTextSources, LocalizationTargetPaths = TArray<FString>(LocalizationTargetPaths), PrioritizedCultureNames = TArray<FString>(PrioritizedCultureNames), LocLoadFlags, Notification = MoveTemp(Notification)]()
	{
		FTextLocalizationManager::Get().LoadLocalizationTargetsForPrioritizedCultures_Sync(AvailableTextSources, LocalizationTargetPaths, PrioritizedCultureNames, LocLoadFlags);
		if (Notification)
		{
			Notification();
		}
	});
}

void FTextLocalizationManager::LoadChunkedLocalizationResources_Sync(TArrayView<const TSharedPtr<ILocalizedTextSource>> AvailableTextSources, const int32 ChunkId, const FString& PakFilename)
{
	LLM_SCOPE(ELLMTag::Localization);
	TRACE_CPUPROFILER_EVENT_SCOPE(FTextLocalizationManager::LoadChunkedLocalizationResources);

	check(ChunkId != INDEX_NONE);

	UE_LOGF(LogTextLocalizationManager, Verbose, "Request to load localization data for chunk %d (from PAK '%ls')", ChunkId, *PakFilename);

	// Skip this request if we've already loaded the data for this chunk via the request for a previous PAK sub-file load notification
	if (LocResTextSource->HasRegisteredChunkId(ChunkId))
	{
		UE_LOGF(LogTextLocalizationManager, Verbose, "Skipped loading localization data for chunk %d (from PAK '%ls') as this chunk has already been processed", ChunkId, *PakFilename);
		return;
	}

	// Nothing to do?
	if (!FTextLocalizationManager::IsDisplayStringSupportEnabled())
	{
		LocResTextSource->RegisterChunkId(ChunkId);

		UE_LOGF(LogTextLocalizationManager, Verbose, "Skipped loading localization data for chunk %d (from PAK '%ls') as display strings are disabled", ChunkId, *PakFilename);
		return;
	}

	// If we're being notified so early that even InitEngineTextLocalization hasn't run, then we can't safely make the queries below as things like GConfig may not be available yet!
	if (!IsInitialized())
	{
		// Track this so that full resource refreshes (eg, changing culture) work as expected
		LocResTextSource->RegisterChunkId(ChunkId);

		UE_LOGF(LogTextLocalizationManager, Verbose, "Skipped loading localization data for chunk %d (from PAK '%ls') as the localization manager isn't ready", ChunkId, *PakFilename);
		return;
	}

	TArray<FString> GameLocalizationPaths;
	GameLocalizationPaths += FPaths::GetGameLocalizationPaths();

	// Note: We only allow game localization targets to be chunked, and the layout is assumed to follow our standard pattern (as used by the localization dashboard and FLocTextHelper)
	TArray<FString> ChunkedLocalizationTargets = FLocalizationResourceTextSource::GetChunkedLocalizationTargets();
	ChunkedLocalizationTargets.RemoveAll([&GameLocalizationPaths](const FString& LocalizationTarget)
	{
		return !GameLocalizationPaths.Contains(FPaths::ProjectContentDir() / TEXT("Localization") / LocalizationTarget);
	});

	// Check to see whether all the required localization data is now available
	// This may not be the case if this PAK was split into multiple sub-files, and the localization data was split between them
	TArray<FString> PrioritizedLocalizationPaths;
	for (const FString& LocalizationTarget : ChunkedLocalizationTargets)
	{
		const FString ChunkedLocalizationTargetName = TextLocalizationResourceUtil::GetLocalizationTargetNameForChunkId(LocalizationTarget, ChunkId);

		FString ChunkedLocalizationTargetPath = FPaths::ProjectContentDir() / TEXT("Localization") / ChunkedLocalizationTargetName;
		if (!IFileManager::Get().DirectoryExists(*ChunkedLocalizationTargetPath))
		{
			UE_LOGF(LogTextLocalizationManager, Verbose, "Skipped loading localization data for chunk %d (from PAK '%ls') as the localization directory for '%ls' was not yet available", ChunkId, *PakFilename, *ChunkedLocalizationTargetName);
			return;
		}

		FTextLocalizationMetaDataResource LocMetaResource;
		{
			const FString LocMetaFilename = ChunkedLocalizationTargetPath / FString::Printf(TEXT("%s.locmeta"), *ChunkedLocalizationTargetName);
			if (!IFileManager::Get().FileExists(*LocMetaFilename))
			{
				UE_LOGF(LogTextLocalizationManager, Verbose, "Skipped loading localization data for chunk %d (from PAK '%ls') as the LocMeta file for '%ls' was not yet available", ChunkId, *PakFilename, *ChunkedLocalizationTargetName);
				return;
			}
			if (!LocMetaResource.LoadFromFile(LocMetaFilename))
			{
				UE_LOGF(LogTextLocalizationManager, Verbose, "Skipped loading localization data for chunk %d (from PAK '%ls') as the LocMeta file for '%ls' failed to load", ChunkId, *PakFilename, *ChunkedLocalizationTargetName);
				return;
			}
		}

		for (const FString& CompiledCulture : LocMetaResource.CompiledCultures)
		{
			const FString LocResFilename = ChunkedLocalizationTargetPath / CompiledCulture / FString::Printf(TEXT("%s.locres"), *ChunkedLocalizationTargetName);
			if (!IFileManager::Get().FileExists(*LocResFilename))
			{
				UE_LOGF(LogTextLocalizationManager, Verbose, "Skipped loading localization data for chunk %d (from PAK '%ls') as the '%ls' LocRes file for '%ls' was not yet available", ChunkId, *PakFilename, *CompiledCulture, *ChunkedLocalizationTargetName);
				return;
			}
		}

		PrioritizedLocalizationPaths.Add(MoveTemp(ChunkedLocalizationTargetPath));
	}

	// Track this so that full resource refreshes (eg, changing culture) work as expected
	LocResTextSource->RegisterChunkId(ChunkId);

	if (!EnumHasAnyFlags(InitializedFlags.load(), ETextLocalizationManagerInitializedFlags::Game))
	{
		// If we've not yet initialized game localization then don't bother patching, as the full initialization path will load the data for this chunk
		return;
	}

	const TArray<FString> PrioritizedCultureNames = FInternationalization::Get().GetPrioritizedCultureNames(FInternationalization::Get().GetCurrentLanguage()->GetName());
	LoadLocalizationTargetsForPrioritizedCultures_Sync(AvailableTextSources, PrioritizedLocalizationPaths, PrioritizedCultureNames, ELocalizationLoadFlags::Native | ELocalizationLoadFlags::Game);
}

void FTextLocalizationManager::LoadChunkedLocalizationResources_Async(const int32 ChunkId, const FString& PakFilename)
{
	QueueAsyncTask([AvailableTextSources = LocalizedTextSources, ChunkId, PakFilename]()
	{
		FTextLocalizationManager::Get().LoadChunkedLocalizationResources_Sync(AvailableTextSources, ChunkId, PakFilename);
	});
}

void FTextLocalizationManager::QueueAsyncTask(TUniqueFunction<void()>&& Task)
{
	if (TextLocalizationManager::AsyncLoadLocalizationData && FTaskGraphInterface::IsRunning())
	{
		if (AsyncLocalizationTask)
		{
			AsyncLocalizationTask = FFunctionGraphTask::CreateAndDispatchWhenReady(MoveTemp(Task), TStatId(), AsyncLocalizationTask);
		}
		else
		{
			AsyncLocalizationTask = FFunctionGraphTask::CreateAndDispatchWhenReady(MoveTemp(Task), TStatId());
		}
	}
	else
	{
		Task();
	}
}

void FTextLocalizationManager::UpdateLiveTable(FTextLocalizationResource&& TextLocalizationResource, const FUpdateLiveTableOptions& UpdateOptions)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FTextLocalizationManager::UpdateLiveTable);
	LLM_SCOPE_BYNAME(TEXT("Localization/DisplayStrings"));

	// Nothing to do?
	if (!FTextLocalizationManager::IsDisplayStringSupportEnabled() || TextLocalizationResource.IsEmpty())
	{
		return;
	}

	// Lock while updating the tables
	{
		FWriteScopeLock ScopeLock(DisplayStringTableRW);

		DisplayStringLookupTable.Reserve(TextLocalizationResource.Entries.Num());

		// Add/update entries
		for (auto& EntryPair : TextLocalizationResource.Entries)
		{
			const FTextId& TextId = EntryPair.Key;
			FTextLocalizationResource::FEntry& NewEntry = EntryPair.Value;

			FDisplayStringEntry* LiveEntry = DisplayStringLookupTable.Find(TextId);
			if (!LiveEntry)
			{
				// Add new entry
				FDisplayStringEntry NewLiveEntry(
					NewEntry.LocResID,						/*LocResID*/
					NewEntry.LocalizationTargetPathId,		/*LocalizationTargetPathId*/
					NewEntry.SourceStringHash,				/*SourceStringHash*/
					NewEntry.LocalizedString.ToSharedRef()	/*String*/
				);

				DisplayStringLookupTable.Emplace(TextId, NewLiveEntry);
				DisplayStringsByLocalizationTargetId.TrackTextId(INDEX_NONE, NewEntry.LocalizationTargetPathId, TextId);
			}
			else if (UpdateOptions.bReplaceExisting)
			{
				// Update existing entry
				LiveEntry->SourceStringHash = NewEntry.SourceStringHash;
				LiveEntry->DisplayString = NewEntry.LocalizedString.ToSharedRef();
#if WITH_EDITORONLY_DATA
				LiveEntry->LocResID = NewEntry.LocResID;
#endif	// WITH_EDITORONLY_DATA
				DisplayStringsByLocalizationTargetId.TrackTextId(LiveEntry->LocalizationTargetPathId, NewEntry.LocalizationTargetPathId, TextId);
				LiveEntry->LocalizationTargetPathId = NewEntry.LocalizationTargetPathId;
			}
		}

		// Note: Do not use TextLocalizationResource after this point as we may have stolen some of its strings
		TextLocalizationResource.Entries.Reset();
	}

	if (UpdateOptions.bDirtyTextRevision)
	{
		DirtyTextRevision();
	}
}

void FTextLocalizationManager::DirtyLocalRevisionForTextId(const FTextId& InTextId)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FTextLocalizationManager::DirtyLocalRevisionForTextId);
	LLM_SCOPE_BYNAME(TEXT("Localization/DisplayStrings"));

	FWriteScopeLock ScopeLock(TextRevisionRW);

	uint16* FoundLocalRevision = LocalTextRevisions.Find(InTextId);
	if (FoundLocalRevision)
	{
		while (++(*FoundLocalRevision) == 0) {} // Zero is special, don't allow an overflow to stay at zero
	}
	else
	{
		LocalTextRevisions.Add(InTextId, 1);
	}
}

void FTextLocalizationManager::DirtyTextRevision()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FTextLocalizationManager::DirtyTextRevision);
	LLM_SCOPE_BYNAME(TEXT("Localization/DisplayStrings"));

	// Lock while updating the data
	{
		FWriteScopeLock ScopeLock(TextRevisionRW);

		while (++TextRevisionCounter == 0) {} // Zero is special, don't allow an overflow to stay at zero
		LocalTextRevisions.Empty();
	}

	if (IsInGameThread())
	{
		OnTextRevisionChangedEvent.Broadcast();
	}
	else
	{
		ExecuteOnGameThread(TEXT("OnTextRevisionChangedEventBroadcastGT"), []()
		{
			FTextLocalizationManager::Get().OnTextRevisionChangedEvent.Broadcast();
		});
	}
}

void FTextLocalizationManager::CacheLocalizedLanguageName(const ELocalizationLoadFlags InLoadFlags)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FTextLocalizationManager::CacheLocalizedLanguageName);

	{
		const TArray<FString> AvailableCultures = GetLocalizedCultureNames(InLoadFlags);
		const TArray<FString> PrioritizedCulturesToMatch = FInternationalization::Get().GetPrioritizedCultureNames(FInternationalization::Get().GetCurrentLanguage()->GetName());
		for (const FString& PrioritizedCulture : PrioritizedCulturesToMatch)
		{
			if (AvailableCultures.Contains(PrioritizedCulture))
			{
				LocalizedLanguageName = PrioritizedCulture;
				return;
			}
		}
	}

	if (EnumHasAnyFlags(InLoadFlags, ELocalizationLoadFlags::Game))
	{
		// If this is a game, use the native culture of the game as the fallback
		LocalizedLanguageName = GetNativeCultureName(ELocalizedTextSourceCategory::Game);
		if (!LocalizedLanguageName.IsEmpty())
		{
			return;
		}
	}
	
	LocalizedLanguageName = TEXT("en");
}

#if WITH_EDITOR
void FTextLocalizationManager::EnableGameLocalizationPreview()
{
	EnableGameLocalizationPreview(GetConfiguredGameLocalizationPreviewLanguage());
}

void FTextLocalizationManager::EnableGameLocalizationPreview(const FString& CultureName)
{
	// This only works in the editor
	if (!GIsEditor)
	{
		return;
	}

	// We need the native game culture to be available for this preview to work correctly
	const FString NativeGameCulture = GetNativeCultureName(ELocalizedTextSourceCategory::Game);
	if (NativeGameCulture.IsEmpty())
	{
		return;
	}

	const FString PreviewCulture = CultureName.IsEmpty() ? NativeGameCulture : CultureName;
	const bool bWillGameLocalizationPreviewBeEnabled = PreviewCulture != NativeGameCulture;
	// We always need to enable this, even when disabling, so that the native language data reloads correctly for plugins (see ShouldForceLoadGameLocalization in FLocalizationTargetDescriptor::ShouldLoadLocalizationTarget)
	// The correct value is set once the loc data has loaded, via the async notification below
	bIsGameLocalizationPreviewEnabled = true;
	bIsLocalizationLocked = IsLocalizationLockedByConfig() || bWillGameLocalizationPreviewBeEnabled;

	TArray<FString> PrioritizedCultureNames;
	if (bWillGameLocalizationPreviewBeEnabled)
	{
		PrioritizedCultureNames = FInternationalization::Get().GetPrioritizedCultureNames(PreviewCulture);
	}
	else
	{
		PrioritizedCultureNames.Add(PreviewCulture);
	}

	ELocalizationLoadFlags LocLoadFlags = ELocalizationLoadFlags::Game | ELocalizationLoadFlags::Additional;
	LocLoadFlags |= (bWillGameLocalizationPreviewBeEnabled ? ELocalizationLoadFlags::Native : ELocalizationLoadFlags::None);

	LoadLocalizationResourcesForPrioritizedCultures_Async(PrioritizedCultureNames, LocLoadFlags, 
		[bWillGameLocalizationPreviewBeEnabled]()
		{
			FTextLocalizationManager::Get().bIsGameLocalizationPreviewEnabled = bWillGameLocalizationPreviewBeEnabled;
		});
}

void FTextLocalizationManager::DisableGameLocalizationPreview()
{
	EnableGameLocalizationPreview(GetNativeCultureName(ELocalizedTextSourceCategory::Game));
}

bool FTextLocalizationManager::IsGameLocalizationPreviewEnabled() const
{
	return bIsGameLocalizationPreviewEnabled;
}

void FTextLocalizationManager::PushAutoEnableGameLocalizationPreview()
{
	++GameLocalizationPreviewAutoEnableCount;
}

void FTextLocalizationManager::PopAutoEnableGameLocalizationPreview()
{
	checkf(GameLocalizationPreviewAutoEnableCount > 0, TEXT("Call to PopAutoEnableGameLocalizationPreview missing corresponding call to PushAutoEnableGameLocalizationPreview!"));
	--GameLocalizationPreviewAutoEnableCount;
}

bool FTextLocalizationManager::ShouldGameLocalizationPreviewAutoEnable() const
{
	return GameLocalizationPreviewAutoEnableCount > 0;
}

void FTextLocalizationManager::ConfigureGameLocalizationPreviewLanguage(const FString& CultureName)
{
	GConfig->SetString(TEXT("Internationalization"), TEXT("PreviewGameLanguage"), *CultureName, GEditorPerProjectIni);
	GConfig->Flush(false, GEditorPerProjectIni);
}

FString FTextLocalizationManager::GetConfiguredGameLocalizationPreviewLanguage() const
{
	return GConfig->GetStr(TEXT("Internationalization"), TEXT("PreviewGameLanguage"), GEditorPerProjectIni);
}

bool FTextLocalizationManager::IsLocalizationLocked() const
{
	return bIsLocalizationLocked;
}
#endif

bool FTextLocalizationManager::ShouldForceLoadGameLocalization() const
{
#if WITH_EDITOR
	if (GIsEditor)
	{
#if UE_IS_COOKED_EDITOR
		return true;
#else	// UE_IS_COOKED_EDITOR
		return bIsGameLocalizationPreviewEnabled || TextLocalizationManager::ForceLoadGameLocalizationInEditor;
#endif	// UE_IS_COOKED_EDITOR
	}
	return false;
#else	// WITH_EDITOR
	return false;
#endif	// WITH_EDITOR
}
