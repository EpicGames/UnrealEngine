// Copyright Epic Games, Inc. All Rights Reserved.

#include "Internationalization/TextLocalizationResource.h"
#include "Internationalization/TextLocalizationResourceVersion.h"
#include "Internationalization/Culture.h"
#include "HAL/FileManager.h"
#include "Misc/App.h"
#include "Misc/Parse.h"
#include "Misc/Paths.h"
#include "Misc/PathViews.h"
#include "Misc/Optional.h"
#include "Misc/ConfigCacheIni.h"
#include "Templates/UniquePtr.h"
#include "Internationalization/Internationalization.h"
#include "Misc/FileHelper.h"
#include "Misc/CommandLine.h"
#include "Serialization/MemoryReader.h"
#include "Trace/Trace.h"
#include "Trace/Trace.inl"

DEFINE_LOG_CATEGORY_STATIC(LogTextLocalizationResource, Log, All);

const FGuid FTextLocalizationResourceVersion::LocMetaMagic = FGuid(0xA14CEE4F, 0x83554868, 0xBD464C6C, 0x7C50DA70);
const FGuid FTextLocalizationResourceVersion::LocResMagic = FGuid(0x7574140E, 0xFC034A67, 0x9D90154A, 0x1B7F37C3);

/** LocMeta files are tiny so we pre-load those by default */
#define PRELOAD_LOCMETA_FILES (1)

/** LocRes files can be quite large, so we won't pre-load those by default */
#define PRELOAD_LOCRES_FILES (0)

namespace TextLocalizationResourceUtil
{

int32 GetLocalizationTargetPathIdFromLocResId(const FTextKey& InLocResID)
{
	int32 LocalizationTargetPathId = INDEX_NONE;
	if (!InLocResID.IsEmpty())
	{
		// LocResID would be "/Path/To/LocalizationTarget/Culture/LocalizationTarget.locres" so trim this back to "/Path/To/LocalizationTarget"
		const FString LocalizationTargetPathStr = InLocResID.ToString();
		FStringView LocalizationTargetPath = LocalizationTargetPathStr;
		LocalizationTargetPath = FPathViews::GetPath(LocalizationTargetPath); // Remove "LocalizationTarget.locres"
		LocalizationTargetPath = FPathViews::GetPath(LocalizationTargetPath); // Remove "Culture"
		LocalizationTargetPathId = FTextLocalizationManager::Get().GetLocalizationTargetPathId(LocalizationTargetPath);
	}
	return LocalizationTargetPathId;
}

}

UE_TRACE_EVENT_BEGIN(Cpu, LoadTextLocalizationMetaDataResourceFromFile, NoSync)
UE_TRACE_EVENT_FIELD(UE::Trace::WideString, Path)
UE_TRACE_EVENT_END()

bool FTextLocalizationMetaDataResource::LoadFromFile(const FString& FilePath)
{
#if CPUPROFILERTRACE_ENABLED
	UE_TRACE_LOG_SCOPED_T(Cpu, LoadTextLocalizationMetaDataResourceFromFile, CpuChannel)
		<< LoadTextLocalizationMetaDataResourceFromFile.Path(*FilePath);
#endif // CPUPROFILERTRACE_ENABLED

	TUniquePtr<FArchive> Reader;
#if PRELOAD_LOCMETA_FILES
	TArray<uint8> FileBytes;
	if (FFileHelper::LoadFileToArray(FileBytes, *FilePath, FILEREAD_Silent))
	{
		Reader = MakeUnique<FMemoryReader>(FileBytes);
	}
#else
	Reader = TUniquePtr<FArchive>(IFileManager::Get().CreateFileReader(*FilePath));
#endif
	if (!Reader)
	{
		UE_LOGF(LogTextLocalizationResource, Log, "LocMeta '%ls' could not be opened for reading!", *FilePath);
		return false;
	}

	bool Success = LoadFromArchive(*Reader, FilePath);
	Success &= Reader->Close();
	return Success;
}

bool FTextLocalizationMetaDataResource::LoadFromArchive(FArchive& Archive, const FString& LocMetaID)
{
	FTextLocalizationResourceVersion::ELocMetaVersion VersionNumber = FTextLocalizationResourceVersion::ELocMetaVersion::Initial;

	// Verify header
	{
		FGuid MagicNumber;
		Archive << MagicNumber;

		if (MagicNumber != FTextLocalizationResourceVersion::LocMetaMagic)
		{
			UE_LOGF(LogTextLocalizationResource, Warning, "LocMeta '%ls' failed the magic number check!", *LocMetaID);
			return false;
		}

		Archive << VersionNumber;
	}

	// Is this LocMeta file too new to load?
	if (VersionNumber > FTextLocalizationResourceVersion::ELocMetaVersion::Latest)
	{
		UE_LOGF(LogTextLocalizationResource, Error, "LocMeta '%ls' is too new to be loaded (File Version: %d, Loader Version: %d)", *LocMetaID, (int32)VersionNumber, (int32)FTextLocalizationResourceVersion::ELocMetaVersion::Latest);
		return false;
	}

	Archive << NativeCulture;
	Archive << NativeLocRes;

	if (VersionNumber >= FTextLocalizationResourceVersion::ELocMetaVersion::AddedCompiledCultures)
	{
		Archive << CompiledCultures;
	}
	else
	{
		CompiledCultures.Reset();
	}

	if (VersionNumber >= FTextLocalizationResourceVersion::ELocMetaVersion::AddedIsUGC)
	{
		Archive << bIsUGC;
	}
	else
	{
		bIsUGC = false;
	}

	return true;
}

bool FTextLocalizationMetaDataResource::SaveToFile(const FString& FilePath)
{
	TUniquePtr<FArchive> Writer(IFileManager::Get().CreateFileWriter(*FilePath));
	if (!Writer)
	{
		UE_LOGF(LogTextLocalizationResource, Log, "LocMeta '%ls' could not be opened for writing!", *FilePath);
		return false;
	}

	bool bSaved = SaveToArchive(*Writer, FilePath);
	bSaved &= Writer->Close();
	return bSaved;
}

bool FTextLocalizationMetaDataResource::SaveToArchive(FArchive& Archive, const FString& LocMetaID)
{
	// Write the header
	{
		FGuid MagicNumber = FTextLocalizationResourceVersion::LocMetaMagic;
		Archive << MagicNumber;

		uint8 VersionNumber = (uint8)FTextLocalizationResourceVersion::ELocMetaVersion::Latest;
		Archive << VersionNumber;
	}

	// Write the native meta-data
	{
		Archive << NativeCulture;
		Archive << NativeLocRes;

		// Added by version: AddedCompiledCultures
		Archive << CompiledCultures;

		// Added by version: AddedIsUGC
		Archive << bIsUGC;
	}

	return true;
}


struct FTextLocalizationResourceString
{
	FTextLocalizationResourceString()
		: String(MakeTextDisplayString(FString()))
	{
	}

	FTextLocalizationResourceString(FString&& InString, int32 InRefCount = 0)
		: String(MakeTextDisplayString(MoveTemp(InString)))
		, RefCount(InRefCount)
	{
	}

	FTextDisplayStringRef String;
	int32 RefCount = 0;

	friend FArchive& operator<<(FArchive& Ar, FTextLocalizationResourceString& A)
	{
		Ar << *A.String;
		Ar << A.RefCount;
		return Ar;
	}
};

void FTextLocalizationResource::AddEntry(const FTextKey& InNamespace, const FTextKey& InKey, const FString& InSourceString, const FString& InLocalizedString, const int32 InPriority, const FTextKey& InLocResID)
{
	AddEntry(InNamespace, InKey, HashString(InSourceString), InLocalizedString, InPriority, InLocResID);
}

void FTextLocalizationResource::AddEntry(const FTextKey& InNamespace, const FTextKey& InKey, const FString& InSourceString, const FTextConstDisplayStringRef& InLocalizedString, const int32 InPriority, const FTextKey& InLocResID)
{
	AddEntry(InNamespace, InKey, HashString(InSourceString), InLocalizedString, InPriority, InLocResID);
}

void FTextLocalizationResource::AddEntry(const FTextKey& InNamespace, const FTextKey& InKey, const uint32 InSourceStringHash, const FString& InLocalizedString, const int32 InPriority, const FTextKey& InLocResID)
{
	AddEntry(InNamespace, InKey, InSourceStringHash, MakeTextDisplayString(CopyTemp(InLocalizedString)), InPriority, InLocResID);
}

void FTextLocalizationResource::AddEntry(const FTextKey& InNamespace, const FTextKey& InKey, const uint32 InSourceStringHash, const FTextConstDisplayStringRef& InLocalizedString, const int32 InPriority, const FTextKey& InLocResID)
{
	AddEntry(InNamespace, InKey, InSourceStringHash, InLocalizedString, InPriority, InLocResID, TextLocalizationResourceUtil::GetLocalizationTargetPathIdFromLocResId(InLocResID));
}

void FTextLocalizationResource::AddEntry(const FTextKey& InNamespace, const FTextKey& InKey, const FString& InSourceString, const FString& InLocalizedString, const int32 InPriority, const FTextKey& InLocResID, const int32 InLocalizationTargetPathID)
{
	AddEntry(InNamespace, InKey, HashString(InSourceString), MakeTextDisplayString(CopyTemp(InLocalizedString)), InPriority, InLocResID, InLocalizationTargetPathID);
}

void FTextLocalizationResource::AddEntry(const FTextKey& InNamespace, const FTextKey& InKey, const uint32 InSourceStringHash, const FTextConstDisplayStringRef& InLocalizedString, const int32 InPriority, const FTextKey& InLocResID, const int32 InLocalizationTargetPathID)
{
	FEntry NewEntry;
	NewEntry.LocResID = InLocResID;
	NewEntry.LocalizationTargetPathId = InLocalizationTargetPathID;
	NewEntry.SourceStringHash = InSourceStringHash;
	NewEntry.LocalizedString = InLocalizedString;
	NewEntry.Priority = InPriority;

	if (FEntry* ExistingEntry = Entries.Find(FTextId(InNamespace, InKey)))
	{
		if (ShouldReplaceEntry(InNamespace, InKey, *ExistingEntry, NewEntry))
		{
			*ExistingEntry = MoveTemp(NewEntry);
		}
	}
	else
	{
		Entries.Emplace(FTextId(InNamespace, InKey), MoveTemp(NewEntry));
	}
}

bool FTextLocalizationResource::IsEmpty() const
{
	return Entries.Num() == 0;
}

void FTextLocalizationResource::LoadFromDirectory(const FString& DirectoryPath, const int32 Priority)
{
	// Find resources in the specified folder.
	TArray<FString> ResourceFileNames;
	if (IFileManager::Get().DirectoryExists(*DirectoryPath))
	{
		IFileManager::Get().FindFiles(ResourceFileNames, *(DirectoryPath / TEXT("*.locres")), true, false);
	}

	for (const FString& ResourceFileName : ResourceFileNames)
	{
		LoadFromFile(FPaths::ConvertRelativePathToFull(DirectoryPath / ResourceFileName), Priority);
	}
}

bool FTextLocalizationResource::LoadFromFile(const FString& FilePath, const int32 Priority)
{
	TUniquePtr<FArchive> Reader;
#if PRELOAD_LOCRES_FILES
	TArray<uint8> FileBytes;
	if (FFileHelper::LoadFileToArray(FileBytes, *FilePath, FILEREAD_Silent))
	{
		Reader = MakeUnique<FMemoryReader>(FileBytes);
	}
#else
	Reader = TUniquePtr<FArchive>(IFileManager::Get().CreateFileReader(*FilePath));
#endif
	if (!Reader)
	{
		UE_LOGF(LogTextLocalizationResource, Log, "LocRes '%ls' could not be opened for reading!", *FilePath);
		return false;
	}

	bool Success = LoadFromArchive(*Reader, FTextKey(FilePath), Priority);
	Success &= Reader->Close();
	return Success;
}

bool FTextLocalizationResource::LoadFromArchive(FArchive& Archive, const FTextKey& LocResID, const int32 Priority)
{
	// Read magic number
	FGuid MagicNumber;
	
	if (Archive.TotalSize() >= sizeof(FGuid))
	{
		Archive << MagicNumber;
	}

	FTextLocalizationResourceVersion::ELocResVersion VersionNumber = FTextLocalizationResourceVersion::ELocResVersion::Legacy;
	if (MagicNumber == FTextLocalizationResourceVersion::LocResMagic)
	{
		Archive << VersionNumber;
	}
	else
	{
		// Legacy LocRes files lack the magic number, assume that's what we're dealing with, and seek back to the start of the file
		Archive.Seek(0);
		UE_LOGF(LogTextLocalizationResource, Warning, "LocRes '%ls' failed the magic number check! Assuming this is a legacy resource (please re-generate your localization resources!)", *LocResID.ToString());
	}

	// Is this LocRes file too new to load?
	if (VersionNumber > FTextLocalizationResourceVersion::ELocResVersion::Latest)
	{
		UE_LOGF(LogTextLocalizationResource, Error, "LocRes '%ls' is too new to be loaded (File Version: %d, Loader Version: %d)", *LocResID.ToString(), (int32)VersionNumber, (int32)FTextLocalizationResourceVersion::ELocResVersion::Latest);
		return false;
	}

	// Read the localized string array
	TArray<FTextLocalizationResourceString> LocalizedStringArray;
	if (VersionNumber >= FTextLocalizationResourceVersion::ELocResVersion::Compact)
	{
		int64 LocalizedStringArrayOffset = INDEX_NONE;
		Archive << LocalizedStringArrayOffset;

		if (LocalizedStringArrayOffset != INDEX_NONE)
		{
			const int64 CurrentFileOffset = Archive.Tell();
			Archive.Seek(LocalizedStringArrayOffset);
			Archive.Precache(LocalizedStringArrayOffset, 0); // Inform the archive that we're going to repeatedly serialize from the current location
			if (VersionNumber >= FTextLocalizationResourceVersion::ELocResVersion::Optimized_CRC32)
			{
				Archive << LocalizedStringArray;
			}
			else
			{
				TArray<FString> TmpLocalizedStringArray;
				Archive << TmpLocalizedStringArray;
				LocalizedStringArray.Reserve(TmpLocalizedStringArray.Num());
				for (FString& LocalizedString : TmpLocalizedStringArray)
				{
					LocalizedStringArray.Emplace(MoveTemp(LocalizedString), INDEX_NONE);
				}
			}
			Archive.Seek(CurrentFileOffset);
			Archive.Precache(CurrentFileOffset, 0); // Inform the archive that we're going to repeatedly serialize from the current location
		}
	}

	// Read entries count
	if (VersionNumber >= FTextLocalizationResourceVersion::ELocResVersion::Optimized_CRC32)
	{
		uint32 EntriesCount;
		Archive << EntriesCount;
		Entries.Reserve(Entries.Num() + FMath::Max(0, (int32)EntriesCount));
	}

	// Read namespace count
	uint32 NamespaceCount;
	Archive << NamespaceCount;

	auto SerializeTextKey = [&VersionNumber, &Archive](FTextKey& InOutTextKey)
	{
		if (VersionNumber >= FTextLocalizationResourceVersion::ELocResVersion::Optimized_CityHash64_UTF16)
		{
			InOutTextKey.SerializeWithHash(Archive);
		}
		else if (VersionNumber == FTextLocalizationResourceVersion::ELocResVersion::Optimized_CRC32)
		{
			InOutTextKey.SerializeDiscardHash(Archive);
		}
		else
		{
			InOutTextKey.SerializeAsString(Archive);
		}
	};

	const int32 LocalizationTargetPathId = TextLocalizationResourceUtil::GetLocalizationTargetPathIdFromLocResId(LocResID);
	for (uint32 i = 0; i < NamespaceCount; ++i)
	{
		// Read namespace
		FTextKey Namespace;
		SerializeTextKey(Namespace);

		// Read key count
		uint32 KeyCount;
		Archive << KeyCount;

		for (uint32 j = 0; j < KeyCount; ++j)
		{
			// Read key
			FTextKey Key;
			SerializeTextKey(Key);

			FEntry NewEntry;
			NewEntry.LocResID = LocResID;
			NewEntry.LocalizationTargetPathId = LocalizationTargetPathId;
			NewEntry.Priority = Priority;

			Archive << NewEntry.SourceStringHash;

			if (VersionNumber >= FTextLocalizationResourceVersion::ELocResVersion::Compact)
			{
				int32 LocalizedStringIndex = INDEX_NONE;
				Archive << LocalizedStringIndex;

				if (LocalizedStringArray.IsValidIndex(LocalizedStringIndex))
				{
					FTextLocalizationResourceString& LocalizedString = LocalizedStringArray[LocalizedStringIndex];
					checkSlow(LocalizedString.RefCount != 0);
					NewEntry.LocalizedString = LocalizedString.String;
					if (LocalizedString.RefCount != INDEX_NONE)
					{
						--LocalizedString.RefCount;
					}
				}
				else
				{
					UE_LOGF(LogTextLocalizationResource, Warning, "LocRes '%ls' has an invalid localized string index for namespace '%ls' and key '%ls'. This entry will have no translation.", *LocResID.ToString(), *Namespace.ToString(), *Key.ToString());
				}
			}
			else
			{
				FString LocalizedString;
				Archive << LocalizedString;
				NewEntry.LocalizedString = MakeTextDisplayString(MoveTemp(LocalizedString));
			}

			if (FEntry* ExistingEntry = Entries.Find(FTextId(Namespace, Key)))
			{
				if (ShouldReplaceEntry(Namespace, Key, *ExistingEntry, NewEntry))
				{
					*ExistingEntry = MoveTemp(NewEntry);
				}
			}
			else
			{
				Entries.Emplace(FTextId(Namespace, Key), MoveTemp(NewEntry));
			}
		}
	}

	return true;
}

bool FTextLocalizationResource::SaveToFile(const FString& FilePath) const
{
	TUniquePtr<FArchive> Writer(IFileManager::Get().CreateFileWriter(*FilePath));
	if (!Writer)
	{
		UE_LOGF(LogTextLocalizationResource, Log, "LocRes '%ls' could not be opened for writing!", *FilePath);
		return false;
	}

	bool bSaved = SaveToArchive(*Writer, FTextKey(FilePath));
	bSaved &= Writer->Close();
	return bSaved;
}

bool FTextLocalizationResource::SaveToArchive(FArchive& Archive, const FTextKey& LocResID) const
{
	// Write the header
	{
		FGuid MagicNumber = FTextLocalizationResourceVersion::LocResMagic;
		Archive << MagicNumber;

		uint8 VersionNumber = (uint8)FTextLocalizationResourceVersion::ELocResVersion::Latest;
		Archive << VersionNumber;
	}

	// Write placeholder offsets for the localized string array
	const int64 LocalizedStringArrayOffset = Archive.Tell();
	{
		int64 DummyOffsetValue = INDEX_NONE;
		Archive << DummyOffsetValue;
	}

	// Arrays tracking localized strings, with a map for efficient look-up of array indices from strings
	TArray<FTextLocalizationResourceString> LocalizedStringArray;
	TMap<FString, int32, FDefaultSetAllocator, FLocKeyMapFuncs<int32>> LocalizedStringMap;

	auto GetLocalizedStringIndex = [&LocalizedStringArray, &LocalizedStringMap](const FString& InString) -> int32
	{
		if (const int32* FoundIndex = LocalizedStringMap.Find(InString))
		{
			++LocalizedStringArray[*FoundIndex].RefCount;
			return *FoundIndex;
		}

		const int32 NewIndex = LocalizedStringArray.Num();
		LocalizedStringArray.Emplace(CopyTemp(InString), 1);
		LocalizedStringMap.Emplace(InString, NewIndex);
		return NewIndex;
	};

	// Rebuild the entries map into a namespace -> keys -> entry map
	typedef TMap<FTextKey, const FEntry*> FKeysTable;
	typedef TMap<FTextKey, FKeysTable> FNamespacesTable;
	FNamespacesTable Namespaces;
	for (const auto& EntryPair : Entries)
	{
		FKeysTable& KeysTable = Namespaces.FindOrAdd(EntryPair.Key.GetNamespace());
		KeysTable.Emplace(EntryPair.Key.GetKey(), &EntryPair.Value);
	}

	// Write entries count
	uint32 EntriesCount = Entries.Num();
	Archive << EntriesCount;

	// Write namespace count
	uint32 NamespaceCount = Namespaces.Num();
	Archive << NamespaceCount;

	// Iterate through namespaces
	for (const auto& NamespaceEntryPair : Namespaces)
	{
		const FTextKey Namespace = NamespaceEntryPair.Key;
		const FKeysTable& KeysTable = NamespaceEntryPair.Value;

		// Write namespace.
		FTextKey NamespaceTmp = Namespace;
		NamespaceTmp.SerializeWithHash(Archive);

		// Write keys count
		uint32 KeyCount = KeysTable.Num();
		Archive << KeyCount;

		// Iterate through keys and values
		for (const auto& KeyEntryPair : KeysTable)
		{
			const FTextKey Key = KeyEntryPair.Key;
			const FEntry* Value = KeyEntryPair.Value;
			check(Value);

			// Write key.
			FTextKey KeyTmp = Key;
			KeyTmp.SerializeWithHash(Archive);

			// Write string entry.
			uint32 SourceStringHash = Value->SourceStringHash;
			Archive << SourceStringHash;

			int32 LocalizedStringIndex = GetLocalizedStringIndex(*Value->LocalizedString);
			Archive << LocalizedStringIndex;
		}
	}

	// Write the localized strings array now
	{
		int64 CurrentFileOffset = Archive.Tell();
		Archive.Seek(LocalizedStringArrayOffset);
		Archive << CurrentFileOffset;
		Archive.Seek(CurrentFileOffset);
		Archive << LocalizedStringArray;
	}

	return true;
}

bool FTextLocalizationResource::ShouldReplaceEntry(const FTextKey& Namespace, const FTextKey& Key, const FEntry& CurrentEntry, const FEntry& NewEntry)
{
	// Note: For priority, smaller numbers are higher priority than bigger numbers

	// Higher priority entries always replace lower priority ones
	if (NewEntry.Priority < CurrentEntry.Priority)
	{
		return true;
	}

	// Lower priority entries never replace higher priority ones
	if (NewEntry.Priority > CurrentEntry.Priority)
	{
		return false;
	}

#if !NO_LOGGING && !UE_BUILD_SHIPPING
	// Equal priority entries won't replace, but may log a conflict
	{
		const bool bDidConflict = CurrentEntry.SourceStringHash != NewEntry.SourceStringHash || !CurrentEntry.LocalizedString->Equals(*NewEntry.LocalizedString, ESearchCase::CaseSensitive);
		if (bDidConflict)
		{
			const FString SummaryMessage = FString::Printf(TEXT("Text translation conflict for namespace \"%s\" and key \"%s\"."),
				*Namespace.ToString(), *Key.ToString());
			const FString DetailsMessage = FString::Printf(TEXT("The current translation is \"%s\" (from \"%s\" and source hash 0x%08x) and the conflicting translation of \"%s\" (from \"%s\" and source hash 0x%08x) will be ignored."), 
				**CurrentEntry.LocalizedString,
				*CurrentEntry.LocResID.ToString(),
				CurrentEntry.SourceStringHash,
				**NewEntry.LocalizedString,
				*NewEntry.LocResID.ToString(),
				NewEntry.SourceStringHash
				);

			static const bool bLoggingConflictsRequested = FParse::Param(FCommandLine::Get(), TEXT("LogLocalizationConflicts"));
			if (bLoggingConflictsRequested)
			{
				// Log the entire message as warning. Include the details in the warning log so they appear in stdout.
				UE_LOGF(LogTextLocalizationResource, Warning, "%ls", *(SummaryMessage + TEXT(" ") + DetailsMessage));
			}
			else if (!GIsBuildMachine)
			{
				// Log just the summary as a warning and hide the rest in the log.
				UE_LOGF(LogTextLocalizationResource, Warning, "%ls", *SummaryMessage);
				UE_LOGF(LogTextLocalizationResource, Log, "%ls", *DetailsMessage);
			}
			else
			{
				// Hide the entire message in the log.
				UE_LOGF(LogTextLocalizationResource, Log, "%ls", *(SummaryMessage + TEXT(" ") + DetailsMessage));
			}
		}
	}
#endif

	return false;
}


namespace TextLocalizationResourceUtil
{
	static TOptional<FString> NativeProjectCultureName;
	static TOptional<FString> NativeEngineCultureName;
#if WITH_EDITOR
	static TOptional<FString> NativeEditorCultureName;
#endif
}



FString TextLocalizationResourceUtil::GetNativeCultureName(const TArray<FString>& InLocalizationPaths)
{
	// Use the native culture of any of the targets on the given paths (it is assumed that all targets for a particular product have the same native culture)
	for (const FString& LocalizationPath : InLocalizationPaths)
	{
		if (!IFileManager::Get().DirectoryExists(*LocalizationPath))
		{
			continue;
		}

		const FString LocMetaFilename = FPaths::GetBaseFilename(LocalizationPath) + TEXT(".locmeta");

		FTextLocalizationMetaDataResource LocMetaResource;
		if (LocMetaResource.LoadFromFile(LocalizationPath / LocMetaFilename))
		{
			return LocMetaResource.NativeCulture;
		}
	}

	return FString();
}

FString TextLocalizationResourceUtil::GetNativeCultureName(const ELocalizedTextSourceCategory InCategory)
{
	switch (InCategory)
	{
	case ELocalizedTextSourceCategory::Game:
		return GetNativeProjectCultureName();
	case ELocalizedTextSourceCategory::Engine:
		return GetNativeEngineCultureName();
	case ELocalizedTextSourceCategory::Editor:
#if WITH_EDITOR
		return GetNativeEditorCultureName();
#else
		break;
#endif
	default:
		checkf(false, TEXT("Unknown ELocalizedTextSourceCategory!"));
		break;
	}
	return FString();
}

FString TextLocalizationResourceUtil::GetNativeProjectCultureName(const bool bSkipCache)
{
	if (!NativeProjectCultureName.IsSet() || bSkipCache)
	{
		NativeProjectCultureName = TextLocalizationResourceUtil::GetNativeCultureName(FPaths::GetGameLocalizationPaths());
	}
	return NativeProjectCultureName.GetValue();
}

void TextLocalizationResourceUtil::ClearNativeProjectCultureName()
{
	NativeProjectCultureName.Reset();
}

FString TextLocalizationResourceUtil::GetNativeEngineCultureName(const bool bSkipCache)
{
	if (!NativeEngineCultureName.IsSet() || bSkipCache)
	{
		NativeEngineCultureName = TextLocalizationResourceUtil::GetNativeCultureName(FPaths::GetEngineLocalizationPaths());
	}
	return NativeEngineCultureName.GetValue();
}

void TextLocalizationResourceUtil::ClearNativeEngineCultureName()
{
	NativeEngineCultureName.Reset();
}

#if WITH_EDITOR
FString TextLocalizationResourceUtil::GetNativeEditorCultureName(const bool bSkipCache)
{
	if (!NativeEditorCultureName.IsSet() || bSkipCache)
	{
		NativeEditorCultureName = TextLocalizationResourceUtil::GetNativeCultureName(FPaths::GetEditorLocalizationPaths());
	}
	return NativeEditorCultureName.GetValue();
}

void TextLocalizationResourceUtil::ClearNativeEditorCultureName()
{
	NativeEditorCultureName.Reset();
}
#endif	// WITH_EDITOR

TArray<FString> TextLocalizationResourceUtil::GetLocalizedCultureNames(const TArray<FString>& InLocalizationPaths)
{
	TArray<FString> CultureNames;

	// Find all unique culture folders that exist in the given paths, skipping the platforms sub-folder
	const FString PlatformFolderName = FPaths::GetPlatformLocalizationFolderName();
	for (const FString& LocalizationPath : InLocalizationPaths)
	{
		const FString LocResFilename = FPaths::GetBaseFilename(LocalizationPath) + TEXT(".locres");
		IFileManager::Get().IterateDirectory(*LocalizationPath, [&CultureNames, &PlatformFolderName, &LocResFilename](const TCHAR* FilenameOrDirectory, bool bIsDirectory) -> bool
		{
			if (bIsDirectory && FCString::Stricmp(FilenameOrDirectory, *PlatformFolderName) != 0)
			{
				const FString LocResPath = FilenameOrDirectory / LocResFilename;
				if (FPaths::FileExists(LocResPath))
				{
					// UE localization resource folders use "en-US" style while ICU uses "en_US"
					const FString LocalizationFolder = FPaths::GetCleanFilename(FilenameOrDirectory);
					const FString CanonicalName = FCulture::GetCanonicalName(LocalizationFolder);
					CultureNames.AddUnique(CanonicalName);
				}
			}
			return true;
		});
	}

	// Remove any cultures that were explicitly disallowed
	FInternationalization& I18N = FInternationalization::Get();
	CultureNames.RemoveAll([&](const FString& InCultureName) -> bool
	{
		return !I18N.IsCultureAllowed(InCultureName);
	});

	return CultureNames;
}

const TArray<FString>& TextLocalizationResourceUtil::GetDisabledLocalizationTargets()
{
	static TArray<FString> DisabledLocalizationTargets;
	static bool bHasInitializedDisabledLocalizationTargets = false;

	if (!bHasInitializedDisabledLocalizationTargets)
	{
		check(GConfig && GConfig->IsReadyForUse());

		const bool bShouldLoadEditor = GIsEditor;
		const bool bShouldLoadGame = FApp::IsGame();

		GConfig->GetArray(TEXT("Internationalization"), TEXT("DisabledLocalizationTargets"), DisabledLocalizationTargets, GEngineIni);

		if (bShouldLoadEditor)
		{
			TArray<FString> EditorArray;
			GConfig->GetArray(TEXT("Internationalization"), TEXT("DisabledLocalizationTargets"), EditorArray, GEditorIni);
			DisabledLocalizationTargets.Append(MoveTemp(EditorArray));
		}

		if (bShouldLoadGame)
		{
			TArray<FString> GameArray;
			GConfig->GetArray(TEXT("Internationalization"), TEXT("DisabledLocalizationTargets"), GameArray, GGameIni);
			DisabledLocalizationTargets.Append(MoveTemp(GameArray));
		}

		bHasInitializedDisabledLocalizationTargets = true;
	}

	return DisabledLocalizationTargets;
}

FString TextLocalizationResourceUtil::GetLocalizationTargetNameForChunkId(const FString& InLocalizationTargetName, const int32 InChunkId)
{
	return InChunkId == INDEX_NONE || InChunkId == 0
		? InLocalizationTargetName
		: FString::Printf(TEXT("%s_locchunk%d"), *InLocalizationTargetName, InChunkId);
}

#undef PRELOAD_LOCMETA_FILES
#undef PRELOAD_LOCRES_FILES
