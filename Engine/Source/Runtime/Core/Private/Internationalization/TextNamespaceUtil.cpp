// Copyright Epic Games, Inc. All Rights Reserved.

#include "Internationalization/TextNamespaceUtil.h"
#include "Misc/Guid.h"

FString TextNamespaceUtil::BuildFullNamespace(const FString& InTextNamespace, const FString& InPackageNamespace, const bool bAlwaysApplyPackageNamespace)
{
	int32 StartMarkerIndex = INDEX_NONE;
	int32 EndMarkerIndex = InTextNamespace.Len() - 1;
	if (InTextNamespace.Len() > 0 && InTextNamespace[EndMarkerIndex] == PackageNamespaceEndMarker && InTextNamespace.FindLastChar(PackageNamespaceStartMarker, StartMarkerIndex))
	{
		FString FullNamespace = InTextNamespace;

		FullNamespace.RemoveAt(StartMarkerIndex + 1, EndMarkerIndex - StartMarkerIndex - 1, EAllowShrinking::No);
		FullNamespace.InsertAt(StartMarkerIndex + 1, InPackageNamespace);

		return FullNamespace;
	}
	else if (bAlwaysApplyPackageNamespace)
	{
		if (InTextNamespace.IsEmpty())
		{
			return FString::Printf(TEXT("%c%s%c"), PackageNamespaceStartMarker, *InPackageNamespace, PackageNamespaceEndMarker);
		}
		else
		{
			return FString::Printf(TEXT("%s %c%s%c"), *InTextNamespace, PackageNamespaceStartMarker, *InPackageNamespace, PackageNamespaceEndMarker);
		}
	}

	return InTextNamespace;
}

FString TextNamespaceUtil::ExtractPackageNamespace(const FString& InTextNamespace)
{
	int32 StartMarkerIndex = INDEX_NONE;
	int32 EndMarkerIndex = InTextNamespace.Len() - 1;
	if (InTextNamespace.Len() > 0 && InTextNamespace[EndMarkerIndex] == PackageNamespaceEndMarker && InTextNamespace.FindLastChar(PackageNamespaceStartMarker, StartMarkerIndex))
	{
		return InTextNamespace.Mid(StartMarkerIndex + 1, EndMarkerIndex - StartMarkerIndex - 1);
	}

	return FString();
}

FString TextNamespaceUtil::StripPackageNamespace(const FString& InTextNamespace)
{
	FString StrippedNamespace = InTextNamespace;
	StripPackageNamespaceInline(StrippedNamespace);
	return StrippedNamespace;
}

void TextNamespaceUtil::StripPackageNamespaceInline(FString& InOutTextNamespace)
{
	int32 StartMarkerIndex = INDEX_NONE;
	int32 EndMarkerIndex = InOutTextNamespace.Len() - 1;
	if (InOutTextNamespace.Len() > 0 && InOutTextNamespace[EndMarkerIndex] == PackageNamespaceEndMarker && InOutTextNamespace.FindLastChar(PackageNamespaceStartMarker, StartMarkerIndex))
	{
		InOutTextNamespace.RemoveAt(StartMarkerIndex, (EndMarkerIndex - StartMarkerIndex) + 1, EAllowShrinking::No);
		InOutTextNamespace.TrimEndInline();
	}
}

FText TextNamespaceUtil::CopyTextToPackage(const FText& InText, const FString& InPackageNamespace, const ETextCopyMethod InCopyMethod, const bool bAlwaysApplyPackageNamespace)
{
	if (InText.IsEmpty() || InText.IsCultureInvariant() || InText.IsFromStringTable())
	{
		// Safe to use these as-is
		return InText;
	}

#if USE_STABLE_LOCALIZATION_KEYS
	if (!InPackageNamespace.IsEmpty() && (InCopyMethod == ETextCopyMethod::NewKey || InCopyMethod == ETextCopyMethod::PreserveKey))
	{
		const TOptional<FString> TextNamespace = FTextInspector::GetNamespace(InText);
		const TOptional<FString> TextKey = FTextInspector::GetKey(InText);

		if (TextNamespace.IsSet() && TextKey.IsSet())
		{
			const FString FullNamespace = BuildFullNamespace(TextNamespace.GetValue(), InPackageNamespace, bAlwaysApplyPackageNamespace);
			if (!TextNamespace.GetValue().Equals(FullNamespace, ESearchCase::CaseSensitive))
			{
				return FText::ChangeKey(FullNamespace, InCopyMethod == ETextCopyMethod::NewKey ? FGuid::NewGuid().ToString() : TextKey.GetValue(), InText);
			}
		}
	}
#endif	// USE_STABLE_LOCALIZATION_KEYS

	return InText;
}

#if USE_STABLE_LOCALIZATION_KEYS

FString TextNamespaceUtil::GetPackageNamespace(FArchive& InArchive)
{
	return InArchive.GetLocalizationNamespace();
}

#endif // USE_STABLE_LOCALIZATION_KEYS
