// Copyright Epic Games, Inc. All Rights Reserved.

#include "Linux/LinuxPlatformFile.h"

IPlatformFile& IPlatformFile::GetPlatformPhysical()
{
	static FUnixPlatformFile Singleton;
	return Singleton;
}
