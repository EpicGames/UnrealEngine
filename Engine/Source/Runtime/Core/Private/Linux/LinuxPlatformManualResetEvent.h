// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Unix/UnixPlatformManualResetEvent.h"

namespace UE { using FPlatformManualResetEvent = UE::HAL::Private::FUnixManualResetEvent; }
