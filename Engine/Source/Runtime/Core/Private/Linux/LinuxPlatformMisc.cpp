// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	LinuxPlatformMisc.cpp: Linux implementations of misc platform functions
=============================================================================*/

#include "Linux/LinuxPlatformMisc.h"
#include "Logging/LogMacros.h"
#include "Misc/CoreDelegates.h"

#include <ifaddrs.h>
#include <net/if.h>
#include <sys/socket.h>
#include <linux/netlink.h>
#include <linux/rtnetlink.h>
#include <unistd.h>

/** Netlink socket file descriptor for receiving network change events. -1 when not initialized. */
static int GNetlinkSocket = -1;

/** Last network connection type that was broadcast, used to suppress redundant notifications. */
static ENetworkConnectionType GLastReportedNetworkType = ENetworkConnectionType::Unknown;

void FLinuxPlatformMisc::PlatformInit()
{
	FUnixPlatformMisc::PlatformInit();
	StartNetworkConnectionMonitoring();
}

void FLinuxPlatformMisc::PlatformTearDown()
{
	StopNetworkConnectionMonitoring();
	FUnixPlatformMisc::PlatformTearDown();
}

ENetworkConnectionType FLinuxPlatformMisc::GetNetworkConnectionType()
{
	struct ifaddrs* InterfaceAddrs = nullptr;
	if (getifaddrs(&InterfaceAddrs) != 0)
	{
		return ENetworkConnectionType::Unknown;
	}

	ENetworkConnectionType Result = ENetworkConnectionType::None;

	for (struct ifaddrs* CurrentAddr = InterfaceAddrs; CurrentAddr != nullptr; CurrentAddr = CurrentAddr->ifa_next)
	{
		// Skip interfaces that are not up and running
		if ((CurrentAddr->ifa_flags & (IFF_UP | IFF_RUNNING)) != (IFF_UP | IFF_RUNNING))
		{
			continue;
		}

		// Skip loopback interfaces
		if (CurrentAddr->ifa_flags & IFF_LOOPBACK)
		{
			continue;
		}

		// Must have a valid address (AF_INET or AF_INET6)
		if (CurrentAddr->ifa_addr == nullptr)
		{
			continue;
		}

		int Family = CurrentAddr->ifa_addr->sa_family;
		if (Family != AF_INET && Family != AF_INET6)
		{
			continue;
		}

		// Found a non-loopback interface that is up and running with a valid address
		Result = ENetworkConnectionType::Ethernet;
		break;
	}

	freeifaddrs(InterfaceAddrs);
	return Result;
}

void FLinuxPlatformMisc::StartNetworkConnectionMonitoring()
{
	check(GNetlinkSocket == -1);

	GNetlinkSocket = socket(AF_NETLINK, SOCK_DGRAM | SOCK_NONBLOCK | SOCK_CLOEXEC, NETLINK_ROUTE);
	if (GNetlinkSocket == -1)
	{
		UE_LOGF(LogInit, Warning, "Failed to create netlink socket for network monitoring (errno=%d)", errno);
		return;
	}

	struct sockaddr_nl Addr;
	FMemory::Memzero(Addr);
	Addr.nl_family = AF_NETLINK;
	Addr.nl_groups = RTMGRP_LINK | RTMGRP_IPV4_IFADDR | RTMGRP_IPV6_IFADDR;

	if (bind(GNetlinkSocket, (struct sockaddr*)&Addr, sizeof(Addr)) < 0)
	{
		UE_LOGF(LogInit, Warning, "Failed to bind netlink socket for network monitoring (errno=%d)", errno);
		close(GNetlinkSocket);
		GNetlinkSocket = -1;
		return;
	}

	// Set the initial network connection status.
	GLastReportedNetworkType = GetNetworkConnectionType();
	SetNetworkConnectionStatus(NetworkConnectionTypeToStatus(GLastReportedNetworkType));
}

void FLinuxPlatformMisc::StopNetworkConnectionMonitoring()
{
	if (GNetlinkSocket != -1)
	{
		close(GNetlinkSocket);
		GNetlinkSocket = -1;
	}
}

void FLinuxPlatformMisc::CheckNetworkConnectionEvents()
{
	if (GNetlinkSocket == -1)
	{
		return;
	}

	// Non-blocking read: drain any pending netlink messages.
	// We only care that something changed, not the specific message contents.
	char Buffer[4096];
	bool bHasEvents = false;

	while (true)
	{
		ssize_t BytesRead = recv(GNetlinkSocket, Buffer, sizeof(Buffer), MSG_DONTWAIT);

		if (BytesRead > 0)
		{
			bHasEvents = true;
			continue;
		}

		if (BytesRead == 0 || errno == EAGAIN || errno == EWOULDBLOCK)
		{
			// No more data available — normal non-blocking termination.
			break;
		}

		if (errno == EINTR)
		{
			// Interrupted by a signal — retry immediately.
			continue;
		}

		if (errno == ENOBUFS || errno == EMSGSIZE)
		{
			// ENOBUFS: kernel receive buffer overflowed — netlink messages were silently dropped.
			// EMSGSIZE: netlink message exceeded the recv buffer — message was truncated.
			// In both cases, force a status check since we may have missed network change events.
			bHasEvents = true;
			break;
		}

		// Real socket error. Close the socket so we don't keep retrying a broken socket every frame.
		UE_LOGF(LogInit, Warning, "Netlink recv() failed (errno=%d). Disabling network connection monitoring.", errno);
		close(GNetlinkSocket);
		GNetlinkSocket = -1;
		return;
	}

	if (bHasEvents)
	{
		const ENetworkConnectionType ConnectionType = GetNetworkConnectionType();

		if (ConnectionType != GLastReportedNetworkType)
		{
			GLastReportedNetworkType = ConnectionType;
			FCoreDelegates::OnNetworkConnectionChanged.Broadcast(ConnectionType);
			SetNetworkConnectionStatus(NetworkConnectionTypeToStatus(ConnectionType));
		}
	}
}
