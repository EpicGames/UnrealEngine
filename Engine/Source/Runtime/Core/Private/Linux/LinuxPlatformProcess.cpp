// Copyright Epic Games, Inc. All Rights Reserved.

#include "Linux/LinuxPlatformProcess.h"
#include "Containers/StringConv.h"
#include "Containers/UnrealString.h"
#include "CoreGlobals.h"
#include "Logging/LogMacros.h"
#include "Misc/Paths.h"
#include "Misc/Parse.h"
#include "HAL/FileManager.h"

const TCHAR* FLinuxPlatformProcess::BaseDir()
{
	static bool bHaveResult = false;
	static TCHAR CachedResult[UNIX_MAX_PATH];
	if (!bHaveResult)
	{
		char SelfPath[UNIX_MAX_PATH] = {0};

		FILE* CmdLineFile = fopen("/proc/self/cmdline", "r");

		if (!CmdLineFile)
		{
			int ErrNo = errno;
			UE_LOGF(LogHAL, Fatal, "fopen() for cmdline failed with errno = %d (%ls)", ErrNo,
				StringCast< TCHAR >(strerror(ErrNo)).Get());
			// unreachable
			return CachedResult;
		}

		// can't lseek or stat /proc/self/cmdline, so we just read in and check each argument
		char* Arg = nullptr;
		size_t Size = 0;
		const char BaseDirCmdArg[] = "-basedir=";
		while(getdelim(&Arg, &Size, 0, CmdLineFile) != -1)
		{
			int Length = strlen(Arg);
			if (FCStringAnsi::Strnicmp(Arg, BaseDirCmdArg, sizeof(BaseDirCmdArg)-1) == 0)
			{
				char* Value = Arg + sizeof(BaseDirCmdArg)-1;
				Length -= sizeof(BaseDirCmdArg)-1;

				FCString::Strncpy(CachedResult, UTF8_TO_TCHAR(Value), UE_ARRAY_COUNT(CachedResult));
				FCString::StrncatTruncateDest(CachedResult, UE_ARRAY_COUNT(CachedResult), TEXT("/"));
				break;
			}
		}
		free(Arg);
		Arg = nullptr;
		Size = 0;
		fclose(CmdLineFile);

		// if not filled in by command line, base it off of the executable location
		if (*CachedResult == '\0')
		{
			if (readlink( "/proc/self/exe", SelfPath, UE_ARRAY_COUNT(SelfPath) - 1) == -1)
			{
				int ErrNo = errno;
				UE_LOGF(LogHAL, Fatal, "readlink() failed with errno = %d (%ls)", ErrNo,
					StringCast< TCHAR >(strerror(ErrNo)).Get());
				// unreachable
				return CachedResult;
			}
			SelfPath[UE_ARRAY_COUNT(SelfPath) - 1] = 0;

			FCString::Strncpy(CachedResult, UTF8_TO_TCHAR(dirname(SelfPath)), UE_ARRAY_COUNT(CachedResult));
			FCString::StrncatTruncateDest(CachedResult, UE_ARRAY_COUNT(CachedResult), TEXT("/"));
		}

#ifdef UE_RELATIVE_BASE_DIR
		FString CollapseResult(CachedResult);

		// this may have been defined at compile time because we are in Restricted, but then we have been staged as a program, and then remapped out of Restricted
		// so if we are already in a Binaries/Linux directory
		if (IFileManager::Get().DirectoryExists(*FPaths::Combine(CollapseResult, UE_RELATIVE_BASE_DIR)))
		{
			CollapseResult /= UE_RELATIVE_BASE_DIR;
		}

		FPaths::CollapseRelativeDirectories(CollapseResult);
		FCString::Strncpy(CachedResult, *CollapseResult, UNIX_MAX_PATH);
#endif

		bHaveResult = true;
	}
	return CachedResult;
}

const TCHAR* FLinuxPlatformProcess::GetBinariesSubdirectory()
{
	if (PLATFORM_CPU_ARM_FAMILY)
	{
		return TEXT("LinuxArm64");
	}

	return TEXT("Linux");
}

FGenericPlatformProcess::EWaitAndForkResult FLinuxPlatformProcess::WaitAndFork()
{
	EWaitAndForkResult RetVal = FUnixPlatformProcess::WaitAndFork();
	if (RetVal == EWaitAndForkResult::Child)
	{
		extern void UELinuxKernelToolsConnection_Init();
		UELinuxKernelToolsConnection_Init();
	}

	return RetVal;
}
