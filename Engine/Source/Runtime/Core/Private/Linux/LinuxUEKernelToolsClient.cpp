// Copyright Epic Games, Inc. All Rights Reserved.
#ifndef ENABLE_LINUX_UE_KERNEL_TOOLS
#define ENABLE_LINUX_UE_KERNEL_TOOLS 0
#endif
#if ENABLE_LINUX_UE_KERNEL_TOOLS

#include "Logging/LogMacros.h"
DEFINE_LOG_CATEGORY_STATIC(LogLinuxUEKernelTools, Log, Log);

#include "LinuxUEKernelToolsShared.h"

#include "HAL/PlatformProcess.h"
#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "Misc/SingleThreadRunnable.h"
#include <poll.h>
#include <sys/eventfd.h>

namespace LinuxUEKernelToolsClient
{
	TAutoConsoleVariable<bool> CVarEnableLinuxKernelToolsIPC(
		TEXT("Linux.EnableLinuxKernelToolsIPC"),
		true,
		TEXT("If true, UELinuxKernelToolsConnection_Init() will create an FLinuxUEKernelToolsClient and read events from LinuxUEKernelTools"),
		ECVF_Default);

	TAutoConsoleVariable<FString> CVarLinuxKernelToolsIPCBasePath(
		TEXT("Linux.LinuxKernelToolsIPCBasePath"),
		"/tmp/LinuxUEKernelTools",
		TEXT("Directory path where Linux kernel tools unix domain sockets will be created"),
		ECVF_Default);
}


struct LinuxUEKernelToolsIPCReceiver : public LinuxUEKernelToolsIPC
{
	sockaddr_un Address;

	LinuxUEKernelToolsIPCReceiver(const char* InBasePath)
		: LinuxUEKernelToolsIPC()
		, Address()
	{
		if (Fd != INVALID_FD)
		{
			fcntl(Fd, F_SETFL, O_NONBLOCK);

			FillSockAddrUn(&Address, InBasePath, getpid());

			// ensure the base dir exists
			if (mkdir(InBasePath, S_IRWXU) && errno != EEXIST)
			{
				UE_LOGF(LogLinuxUEKernelTools, Error, "mkdir call failed, errno = %i (%hs)", errno,
					   strerror(errno));
			}

			UE_LOGF(LogLinuxUEKernelTools, Verbose, "Binding to %hs", Address.sun_path);
			if (bind(Fd, (const struct sockaddr *) &Address, sizeof(Address)) != 0)
			{
				UE_LOGF(LogLinuxUEKernelTools, Error, "Failed to bind socket AF_UNIX, errno = %i (%hs)", errno,
					   strerror(errno));
			}
		}
	}

	~LinuxUEKernelToolsIPCReceiver()
	{
		unlink(Address.sun_path);
	}

	int Receive(BPFEvent* EventBuffer, size_t MaxEventCount)
	{
		if (Fd != INVALID_FD && EventBuffer != nullptr && MaxEventCount)
		{
			const int Result = recvfrom(Fd, EventBuffer, MaxEventCount * sizeof(BPFEvent), 0, nullptr, nullptr);
			if (Result >= 0)
			{
				return Result / sizeof(BPFEvent);
			}
			if (errno != EAGAIN && errno != EWOULDBLOCK)
			{
				UE_LOGF(LogLinuxUEKernelTools, Error, "recvfrom failed, %d:%s", errno, strerror(errno));
			}
		}
		return 0;
	}
};

class FLinuxUEKernelToolsClient : public FRunnable, public FSingleThreadRunnable
{
public:
	FLinuxUEKernelToolsClient()
		: IPC(StringCast<ANSICHAR>(*LinuxUEKernelToolsClient::CVarLinuxKernelToolsIPCBasePath.GetValueOnGameThread()).Get())
		, StopFd(eventfd(0, EFD_NONBLOCK))
		, PageSize(getpagesize())
	{
		UE_LOGF(LogLinuxUEKernelTools, Log, "Creating FLinuxUEKernelToolsClient");
	}

	virtual bool Init() override
	{
		return IPC.Fd != INVALID_FD;
	}

	virtual uint32 Run() override
	{
		pollfd PollFds[2] = {
			{ .fd = IPC.Fd, .events = POLLIN },
			{ .fd = StopFd, .events = POLLIN }
		};

		while (true)
		{
			if (poll(PollFds, 2, -1) > 0)
			{
				if (PollFds[0].revents & POLLIN)
				{
					while (true)
					{
						const int EventCount = IPC.Receive(EventBuffer, LinuxUEKernelToolsIPC::MAX_EVENTS_PER_PACKET);
						for (int I = 0; I < EventCount; ++I)
						{
							HandleEvent(EventBuffer[I]);
						}
						if (EventCount < LinuxUEKernelToolsIPC::MAX_EVENTS_PER_PACKET)
						{
							break;
						}
					}
				}

				if (PollFds[1].revents & POLLIN)
				{
					// Stop() has been called
					close(StopFd);
					break;
				}
			}
		}

		return 0;
	}

	virtual void Tick() override
	{
		while (true)
		{
			const int EventCount = IPC.Receive(EventBuffer, LinuxUEKernelToolsIPC::MAX_EVENTS_PER_PACKET);
			for (int I = 0; I < EventCount; ++I)
			{
				HandleEvent(EventBuffer[I]);
			}
			if (EventCount < LinuxUEKernelToolsIPC::MAX_EVENTS_PER_PACKET)
			{
				break;
			}
		}
	}

	virtual void Stop() override
	{
		uint64 Value = 1; // this has to be 8 bytes or the kernel will reject it
		if (write(StopFd, &Value, 8) == -1)
		{
			UE_LOGF(LogLinuxUEKernelTools, Error, "write to StopFd failed, %d:%s", errno, strerror(errno));
		}
	}

	virtual class FSingleThreadRunnable* GetSingleThreadInterface() override { return this; }

private:
	BPFEvent EventBuffer[LinuxUEKernelToolsIPC::MAX_EVENTS_PER_PACKET];
	LinuxUEKernelToolsIPCReceiver IPC;
	int StopFd;
	size_t PageSize;

	void HandleEvent(const BPFEvent& Event)
	{
		switch (Event.Type)
		{
			case BPFEventCOW:
				FPlatformMemory::IncSharedMemoryCOWBytes(PageSize);
				break;
			default:
				break;
		}
	}
};

static FRunnableThread* ClientThread;

void UELinuxKernelToolsConnection_Init()
{
	if (!LinuxUEKernelToolsClient::CVarEnableLinuxKernelToolsIPC.GetValueOnGameThread())
	{
		return;
	}

	ensure(ClientThread == nullptr);

	ClientThread = FRunnableThread::Create(new FLinuxUEKernelToolsClient(), TEXT("FLinuxUEKernelToolsClient"), 0, TPri_Normal);
}
#else
void UELinuxKernelToolsConnection_Init()
{
	// no-op
}
#endif