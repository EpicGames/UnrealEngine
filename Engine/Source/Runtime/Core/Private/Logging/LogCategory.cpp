// Copyright Epic Games, Inc. All Rights Reserved.

#include "Logging/LogCategory.h"

#include "Containers/UnrealString.h"
#include "CoreGlobals.h"
#include "HAL/PlatformMisc.h"
#include "Logging/LogSuppressionInterface.h"
#include "Logging/LogTrace.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CoreDelegates.h"
#include "Misc/OutputDeviceRedirector.h"

FLogCategoryBase::FLogCategoryBase(const FLogCategoryName& InCategoryName, ELogVerbosity::Type InDefaultVerbosity, ELogVerbosity::Type InCompileTimeVerbosity)
	: DefaultVerbosity(InDefaultVerbosity)
	, CompileTimeVerbosity(InCompileTimeVerbosity)
	, CategoryName(InCategoryName)
{
	TRACE_LOG_CATEGORY(this, *FName(CategoryName).ToString(), InDefaultVerbosity);
	ResetFromDefault();
	if (CompileTimeVerbosity > ELogVerbosity::NoLogging)
	{
		FLogSuppressionInterface::Get().AssociateSuppress(this);
	}
	checkSlow(!(Verbosity & ELogVerbosity::BreakOnLog)); // this bit is factored out of this variable, always
}

FLogCategoryBase::FLogCategoryBase(const TCHAR* InCategoryName, ELogVerbosity::Type InDefaultVerbosity, ELogVerbosity::Type InCompileTimeVerbosity)
	: FLogCategoryBase(FLogCategoryName{InCategoryName}, InDefaultVerbosity, InCompileTimeVerbosity)
{
}

FLogCategoryBase::~FLogCategoryBase()
{
	checkSlow(!(Verbosity & ELogVerbosity::BreakOnLog)); // this bit is factored out of this variable, always
	if (CompileTimeVerbosity > ELogVerbosity::NoLogging)
	{
		if (FLogSuppressionInterface* Singleton = FLogSuppressionInterface::TryGet())
		{
			Singleton->DisassociateSuppress(this);
		}
	}
}

void FLogCategoryBase::SetVerbosity(ELogVerbosity::Type InNewVerbosity)
{
	const ELogVerbosity::Type OldVerbosity = Verbosity;
	const ELogVerbosity::Type NewVerbosity = (ELogVerbosity::Type)(InNewVerbosity & ELogVerbosity::VerbosityMask);
	// regularize the verbosity to be at most whatever we were compiled with
	if (ensureMsgf(NewVerbosity <= CompileTimeVerbosity, TEXT("Log category '%s' tried to set verbosity (%s) higher than its compile-time verbosity level (%s)"), *GetCategoryName().ToString(), ToString(NewVerbosity), ToString(CompileTimeVerbosity)))
	{
		Verbosity = NewVerbosity;
	}
	else
	{
		Verbosity = CompileTimeVerbosity;
	}
	DebugBreakOnLog = !!(InNewVerbosity & ELogVerbosity::BreakOnLog);
	checkSlow(!(Verbosity & ELogVerbosity::BreakOnLog)); // this bit is factored out of this variable, always

	if (OldVerbosity != Verbosity)
	{
		FCoreDelegates::OnLogVerbosityChanged.Broadcast(GetCategoryName(), OldVerbosity, Verbosity);
	}
}

void FLogCategoryBase::ResetFromDefault()
{
	// regularize the default verbosity to be at most whatever we were compiled with
	SetVerbosity(ELogVerbosity::Type(DefaultVerbosity));
}


void FLogCategoryBase::PostTrigger(ELogVerbosity::Type VerbosityLevel)
{
	checkSlow(!(Verbosity & ELogVerbosity::BreakOnLog)); // this bit is factored out of this variable, always
	check(VerbosityLevel <= CompileTimeVerbosity); // we should have never gotten here, the compile-time version should ALWAYS be checked first
	if (DebugBreakOnLog || (VerbosityLevel & ELogVerbosity::BreakOnLog))  // we break if either the suppression level on this message is set to break or this log statement is set to break
	{
		GLog->FlushThreadedLogs();
		DebugBreakOnLog = false; // toggle this off automatically
		UE_DEBUG_BREAK();
	}
}
