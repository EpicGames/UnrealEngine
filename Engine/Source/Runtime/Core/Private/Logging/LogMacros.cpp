// Copyright Epic Games, Inc. All Rights Reserved.

#include "Logging/LogMacros.h"
#include "CoreGlobals.h"
#include "HAL/Platform.h"
#include "HAL/PlatformMallocCrash.h"
#include "Misc/ScopeLock.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Misc/FeedbackContext.h"
#include "Misc/VarargsHelper.h"
#include "Stats/Stats.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "AutoRTFM.h"

void StaticFailDebugV(const TCHAR* Error, const ANSICHAR* File, int32 Line, void* ProgramCounter, const TCHAR* DescriptionFormat, va_list DescriptionArgs);

CSV_DEFINE_CATEGORY(FMsgLogf, true);

UE_AUTORTFM_ALWAYS_OPEN
void FMsg::LogfImplV(const ANSICHAR* File, int32 Line, const FLogCategoryName& Category, ELogVerbosity::Type Verbosity, void* ReturnAddress, const TCHAR* Fmt, va_list Args)
{
	// This function always executes in the open, because our loggers and crash handlers are not AutoRTFM-safe.
	if (LIKELY(Verbosity != ELogVerbosity::Fatal))
	{
		// SetColour is routed to GWarn just like the other verbosities and handled in the
		// device that does the actual printing.
		FOutputDevice* LogOverride = nullptr;
		switch (Verbosity)
		{
		case ELogVerbosity::Error:
		case ELogVerbosity::Warning:
		case ELogVerbosity::Display:
		case ELogVerbosity::SetColor:
			LogOverride = GWarn;
			break;
		default:
			break;
		}

		GrowableLogfV(Fmt, Args, [&](const TCHAR* Buffer)
		{
			LogOverride ? LogOverride->Log(Category, Verbosity, Buffer)
		                : GLog->RedirectLog(Category, Verbosity, Buffer);
		});
	}
	else
	{
		va_list ArgsCopy;
		va_copy(ArgsCopy, Args);
		StaticFailDebugV(TEXT("Fatal error:"), File, Line, ReturnAddress, Fmt, ArgsCopy);
		va_end(ArgsCopy);

		FDebug::AssertFailedV("", File, Line, Fmt, Args);
	}
}

void FMsg::LogfImpl(const ANSICHAR* File, int32 Line, const FLogCategoryName& Category, ELogVerbosity::Type Verbosity, const TCHAR* Fmt, ...)
{
#if !NO_LOGGING
	va_list Args;
	va_start(Args, Fmt);
	LogfImplV(File, Line, Category, Verbosity, PLATFORM_RETURN_ADDRESS(), Fmt, Args);
	va_end(Args);
#endif
}

void FMsg::LogV(const ANSICHAR* File, int32 Line, const FLogCategoryName& Category, ELogVerbosity::Type Verbosity, const TCHAR* Fmt, va_list Args)
{
#if !NO_LOGGING
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FMsgLogf);

	if (LIKELY(Verbosity != ELogVerbosity::Fatal))
	{
		TStringBuilder<512> Buffer;
		Buffer.AppendV(Fmt, Args);
		const TCHAR* Message = *Buffer;
		FOutputDevice* OutputDevice = GLog;
		switch (Verbosity)
		{
		case ELogVerbosity::Error:
		case ELogVerbosity::Warning:
		case ELogVerbosity::Display:
		case ELogVerbosity::SetColor:
			if (GWarn != nullptr)
			{
				OutputDevice = GWarn;
			}
			break;
		default:
			break;
		}

		// Logging is always done in the open as we want logs even with transactionalized code.
		UE_AUTORTFM_OPEN
		{
			OutputDevice->Serialize(Message, Verbosity, Category);
		};

#if CSV_PROFILER_STATS
		// Only update the CSV stat if we're not crashing, otherwise things can get messy
		if (LIKELY(!FPlatformMallocCrash::IsActive()))
		{
			CSV_CUSTOM_STAT(FMsgLogf, FMsgLogfCount, 1, ECsvCustomStatOp::Accumulate);
		}
#endif

	}
	else
	{
		StaticFailDebugV(TEXT("Fatal error:"), File, Line, PLATFORM_RETURN_ADDRESS(), Fmt, Args);
	}
#endif
}

void FMsg::Logf_InternalImpl(const ANSICHAR* File, int32 Line, const FLogCategoryName& Category, ELogVerbosity::Type Verbosity, const TCHAR* Fmt, ...)
{
#if !NO_LOGGING
	va_list Args;
	va_start(Args, Fmt);
	LogV(File, Line, Category, Verbosity, Fmt, Args);
	va_end(Args);
#endif
}

/** Sends a formatted message to a remote tool. */
void VARARGS FMsg::SendNotificationStringfImpl( const TCHAR *Fmt, ... )
{
	GROWABLE_LOGF(SendNotificationString(Buffer));
}

void FMsg::SendNotificationString( const TCHAR* Message )
{
	FPlatformMisc::LowLevelOutputDebugString(Message);
}
