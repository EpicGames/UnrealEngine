// Copyright Epic Games, Inc. All Rights Reserved.

#include "Logging/LogScopedCategoryAndVerbosityOverride.h"

#include "AutoRTFM.h"
#include "CoreTypes.h"
#include "HAL/PlatformTLS.h"

FLogScopedCategoryAndVerbosityOverride::FLogScopedCategoryAndVerbosityOverride(FName Category, ELogVerbosity::Type Verbosity)
{
	FOverride* TLS = GetTLSCurrent();
	Backup = *TLS;
	*TLS = FOverride(Category, Verbosity);
}

FLogScopedCategoryAndVerbosityOverride::~FLogScopedCategoryAndVerbosityOverride()
{
	FOverride* TLS = GetTLSCurrent();
	*TLS = Backup;
}

// OverrideTLSID is stored as a local static to make sure we initialize before use, even
// it it happens during the static initialization. Otherwise we could end up setting
// values to the 0 TLS slot which doesn't belong to us.
uint32 GetOverrideTLSID()
{
	static uint32 OverrrideTLSID = FPlatformTLS::AllocTlsSlot();

	return OverrrideTLSID;
}

UE_AUTORTFM_ALWAYS_OPEN
FLogScopedCategoryAndVerbosityOverride::FOverride* FLogScopedCategoryAndVerbosityOverride::GetTLSCurrent()
{
	FOverride* TLS = (FOverride*)FPlatformTLS::GetTlsValue(GetOverrideTLSID());
	if (!TLS)
	{
		TLS = new FOverride;
		FPlatformTLS::SetTlsValue(GetOverrideTLSID(), TLS);
	}	
	return TLS;
}

