// Copyright Epic Games, Inc. All Rights Reserved.

#include "Logging/LogScopedVerbosityOverride.h"
#include "Misc/AssertionMacros.h"
#include "Logging/LogCategory.h"

/** Back up the existing verbosity for the category then sets new verbosity.*/
FLogScopedVerbosityOverride::FLogScopedVerbosityOverride(FLogCategoryBase * Category, ELogVerbosity::Type Verbosity)
	:	SavedCategory(Category)
{
	check (SavedCategory);
	SavedVerbosity = SavedCategory->GetVerbosity();
	SavedCategory->SetVerbosity(Verbosity);
}

/** Restore the verbosity overrides for the category to the previous value.*/
FLogScopedVerbosityOverride::~FLogScopedVerbosityOverride()
{
	SavedCategory->SetVerbosity(SavedVerbosity);
}

