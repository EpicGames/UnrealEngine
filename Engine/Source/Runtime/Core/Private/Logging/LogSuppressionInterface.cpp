// Copyright Epic Games, Inc. All Rights Reserved.

#include "Logging/LogSuppressionInterface.h"
#include "Misc/AssertionMacros.h"
#include "Math/UnrealMathUtility.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "UObject/NameTypes.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Misc/LazySingleton.h"
#include "Misc/Parse.h"
#include "Containers/Map.h"
#include "Misc/CoreMisc.h"
#include "Misc/CoreDelegates.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/OutputDeviceHelper.h"
#include "Modules/ModuleManager.h"

namespace
{
	struct FLogCategoryPtrs
	{
		explicit FLogCategoryPtrs(const FString& InName, ELogVerbosity::Type InVerbosity, bool InPostfix)
		: Name     (InName)
		, Verbosity(InVerbosity)
		, Postfix  (InPostfix)
		{
		}

		FString             Name;
		ELogVerbosity::Type Verbosity;
		bool                Postfix;

		friend bool operator<(const FLogCategoryPtrs& Lhs, const FLogCategoryPtrs& Rhs)
		{
			return Lhs.Name < Rhs.Name;
		}
	};
}


/** A "fake" logging category that is used as a proxy for changing all categories **/
static const FLogCategoryBase& GetGlobalVerbosity()
{
	static FLogCategoryBase GlobalVerbosity(TEXT("Global"), ELogVerbosity::All, ELogVerbosity::All);
	return GlobalVerbosity;
}
/** A "fake" logging category that is used as a proxy for changing the default of all categories at boot time. **/
static const FLogCategoryBase& GetBootGlobalVerbosity()
{
	static FLogCategoryBase BootGlobalVerbosity(TEXT("BootGlobal"), ELogVerbosity::All, ELogVerbosity::All);
	return BootGlobalVerbosity;
}


/** Log suppression system implementation **/
class FLogSuppressionImplementation: public FLogSuppressionInterface, private FSelfRegisteringExec
{
	enum { NumInlineCategories = 256 };

	/** Categories waiting to be added to ReverseAssociations */
	TArray<FLogCategoryBase*, TInlineAllocator<NumInlineCategories>> PendingAssociations;

	/** Associates a category name with a set of category pointers  **/
	TMultiMap<FName, FLogCategoryBase*> ReverseAssociations;
	/** Set of verbosity and break values that were set at boot time. **/
	TMap<FName, uint8> BootAssociations;
	/** For a given category stores the last non-zero verbosity...to support toggling without losing the specific verbosity level **/
	TMap<FName, uint8> ToggleAssociations;

	bool bInitialized = false;

	/**
	 * Process a "[cat] only" string command to the logging suppression system
	 * @param CmdString, string to process
	 * @return true if CmdString was a "[cat] only" string command, false otherwise
	 */
	bool ProcessLogOnly(const FString& CmdString, FOutputDevice& Ar)
	{
		TArray<FString> CommandParts;
		CmdString.ParseIntoArrayWS(CommandParts);
		if (CommandParts.Num() <= 1)
		{
			return false;
		}
		
		static FName NAME_Only("only");
		if (NAME_Only != FName(*CommandParts[1]))
		{
			return false;
		}
		
		FName LogCategory = FName(*CommandParts[0]);
		static const FString OffString = FString(" off");
		static const FString OnString = FString(" Verbose");
		for (TPair<FName, FLogCategoryBase*> It : ReverseAssociations)
		{
			FName Name = It.Key;
			if (Name == LogCategory)
			{
				ProcessCmdString(Name.ToString() + OnString);
				FLogCategoryBase* Verb = It.Value;
				Ar.Logf(TEXT("%s is now %s"), *CommandParts[0], ToString(Verb ? ELogVerbosity::Type(Verb->Verbosity) : ELogVerbosity::Verbose));
			}
			else
			{
				ProcessCmdString(Name.ToString() + OffString);
			}
		}

		Ar.Logf(TEXT("Disabling other logs finished"));
		return true;
	}

	/** 
	 * Process a string command to the logging suppression system 
	 * @param CmdString, string to process
	 * @param FromBoot, if true, this is a boot time command, and is handled differently
	 */
	void ProcessCmdString(const FString& CmdString, bool FromBoot = false)
	{
		// How to use the log command : `log <category> <verbosity>
		// e.g., Turn off all logging : `log global none
		// e.g., Set specific filter  : `log logshaders verbose
		// e.g., Combo command        : `log global none, log logshaders verbose

		static FName NAME_BootGlobal("BootGlobal");
		static FName NAME_Reset("Reset");
		FString Cmds = CmdString;
		Cmds = Cmds.TrimStart().TrimQuotes().TrimStart();
		TArray<FString> SubCmds;
		Cmds.ParseIntoArray(SubCmds, TEXT(","), true);
		for (int32 Index = 0; Index < SubCmds.Num(); Index++)
		{
			static const auto& LogString = TEXT("Log ");
			FString Command = SubCmds[Index].TrimStart();
			if (Command.StartsWith(LogString))
			{
				Command.RightInline(Command.Len() - (UE_ARRAY_COUNT(LogString) - 1), EAllowShrinking::No);
			}
			TArray<FString> CommandParts;
			Command.ParseIntoArrayWS(CommandParts);
			if (CommandParts.Num() < 1)
			{
				continue;
			}
			FName Category(*CommandParts[0]);
			if (Category == NAME_Global && FromBoot)
			{
				Category = NAME_BootGlobal; // the boot time global is a special one, since we want things like "log global none, log logshaders verbose"
			}
			TArray<FLogCategoryBase*> CategoryVerbosities;
			uint8 Value = 0;
			if (FromBoot)
			{
				// now maybe this was already set at boot, in which case we override what it had
				uint8* Boot = BootAssociations.Find(Category);
				if (Boot)
				{
					Value = *Boot;
				}
				else
				{
					// see if we had a boot global override
					Boot = BootAssociations.Find(NAME_BootGlobal);
					if (Boot)
					{
						Value = *Boot;
					}
				}
			}
			else
			{
				for (TMultiMap<FName, FLogCategoryBase*>::TKeyIterator It(ReverseAssociations, Category); It; ++It)
				{
					checkSlow(!(It.Value()->Verbosity & ELogVerbosity::BreakOnLog)); // this bit is factored out of this variable, always
					Value = It.Value()->Verbosity | (It.Value()->DebugBreakOnLog ? ELogVerbosity::BreakOnLog : 0);
					CategoryVerbosities.Add(It.Value());
				}					
			}

			const uint8 OriginalVerbosityLevel = Value & ELogVerbosity::VerbosityMask;
			if (CommandParts.Num() == 1)
			{
				// only possibility is the reset and toggle command which is meaningless at boot
				if (!FromBoot)
				{
					if (Category == NAME_Reset)
					{
						for (TPair<FName, FLogCategoryBase*> It : ReverseAssociations)
						{
							FLogCategoryBase* Verb = It.Value;
							Verb->ResetFromDefault();
							// store off the last non-zero one for toggle
							checkSlow(!(Verb->Verbosity & ELogVerbosity::BreakOnLog)); // this bit is factored out of this variable, always
							if (Verb->Verbosity)
							{
								// currently on, store this in the pending and clear it
								ToggleAssociations.Add(Category, Verb->Verbosity);
							}
						}
					}
					else
					{
						if (Value & ELogVerbosity::VerbosityMask)
						{
							// currently on, toggle it
							Value = Value & ~ELogVerbosity::VerbosityMask;
						}
						else
						{
							// try to get a non-zero value from the toggle backup
							uint8* Toggle = ToggleAssociations.Find(Category);
							if (Toggle && *Toggle)
							{
								Value |= *Toggle;
							}
							else
							{
								Value |= ELogVerbosity::All;
							}
						}
					}
				}
			}
			else
			{
				// now we have the current value, lets change it!
				for (int32 PartIndex = 1; PartIndex < CommandParts.Num(); PartIndex++)
				{
					FName CmdToken = FName(*CommandParts[PartIndex]);
					static FName NAME_Verbose("Verbose");
					static FName NAME_VeryVerbose("VeryVerbose");
					static FName NAME_On("On");
					static FName NAME_Off("Off");
					static FName NAME_Break("Break");
					static FName NAME_Fatal("Fatal");
					static FName NAME_Log("Log");
					static FName NAME_Display("Display");
					if (CmdToken == NAME_None)
					{
						Value &= ~ELogVerbosity::VerbosityMask;
						Value |= ELogVerbosity::Fatal;
					}
					else if (CmdToken == NAME_Fatal)
					{
						Value &= ~ELogVerbosity::VerbosityMask;
						Value |= ELogVerbosity::Fatal;
					}
					else if (CmdToken == NAME_Error)
					{
						Value &= ~ELogVerbosity::VerbosityMask;
						Value |= ELogVerbosity::Error;
					}
					else if (CmdToken == NAME_Warning)
					{
						Value &= ~ELogVerbosity::VerbosityMask;
						Value |= ELogVerbosity::Warning;
					}
					else if (CmdToken == NAME_Log)
					{
						Value &= ~ELogVerbosity::VerbosityMask;
						Value |= ELogVerbosity::Log;
					}
					else if (CmdToken == NAME_Display)
					{
						Value &= ~ELogVerbosity::VerbosityMask;
						Value |= ELogVerbosity::Display;
					}
					else if (CmdToken == NAME_Verbose)
					{
						Value &= ~ELogVerbosity::VerbosityMask;
						Value |= ELogVerbosity::Verbose;
					}
					else if (CmdToken == NAME_VeryVerbose || CmdToken == NAME_All)
					{
						Value &= ~ELogVerbosity::VerbosityMask;
						Value |= ELogVerbosity::VeryVerbose;
					}
					else if (CmdToken == NAME_Default)
					{
						if (CategoryVerbosities.Num() && !FromBoot)
						{
							Value = CategoryVerbosities[0]->DefaultVerbosity;
						}
					}
					else if (CmdToken == NAME_Off)
					{
						Value &= ~ELogVerbosity::VerbosityMask;
						Value |= ELogVerbosity::Fatal;
					}
					else if (CmdToken == NAME_On)
					{
						Value &= ~ELogVerbosity::VerbosityMask;
							// try to get a non-zero value from the toggle backup
							uint8* Toggle = ToggleAssociations.Find(Category);
							if (Toggle && *Toggle)
							{
								Value |= *Toggle;
							}
							else
							{
								Value |= ELogVerbosity::All;
							}
					}
					else if (CmdToken == NAME_Break)
					{
						Value ^= ELogVerbosity::BreakOnLog;
					}
				}
			}
			if (Category != NAME_Reset)
			{
				if (FromBoot)
				{
					if (Category == NAME_BootGlobal)
					{
						// changing the global at boot removes everything set up so far
						BootAssociations.Empty();
					}
					BootAssociations.Add(Category, Value);
				}
				else
				{
					for (int32 CategoryIndex = 0; CategoryIndex < CategoryVerbosities.Num(); CategoryIndex++)
					{
						FLogCategoryBase* Verb = CategoryVerbosities[CategoryIndex];
						Verb->SetVerbosity(ELogVerbosity::Type(Value));
					}
					if (Category == NAME_Global)
					{
						// if this was a global change, we need to change them all
						ApplyGlobalChanges();
					}
				}
				// store off the last non-zero one for toggle
				const uint8 VerbosityLevel = Value & ELogVerbosity::VerbosityMask;
				if (VerbosityLevel)
				{
					// currently on, store this in the pending and clear it
					ToggleAssociations.Add(Category, VerbosityLevel);

					// Tattle on configs & other paths that raise the level after boot so we can quickly rule out code defaults for log spam
					if ((OriginalVerbosityLevel < ELogVerbosity::Verbose) && (VerbosityLevel >= ELogVerbosity::Verbose))
					{
						UE_LOGF(LogHAL, Log, "Log category %ls verbosity has been raised to %ls.", *Category.ToString(), ToString(static_cast<ELogVerbosity::Type>(VerbosityLevel)));
					}
				}
			}
		}
	}
	/** Called after a change is made to the global verbosity...Iterates over all logging categories and adjusts them accordingly **/
	void ApplyGlobalChanges()
	{
		const FLogCategoryBase& GlobalVerbosity = GetGlobalVerbosity();

		static ELogVerbosity::Type LastGlobalVerbosity = ELogVerbosity::All;
		bool bVerbosityGoingUp = GlobalVerbosity.Verbosity > LastGlobalVerbosity;
		bool bVerbosityGoingDown = GlobalVerbosity.Verbosity < LastGlobalVerbosity;
		checkSlow(!(GlobalVerbosity.Verbosity & ELogVerbosity::BreakOnLog)); // this bit is factored out of this variable, always
		LastGlobalVerbosity = GlobalVerbosity.Verbosity;

		static bool bOldGlobalBreakValue = false;
		bool bForceBreak = (!GlobalVerbosity.DebugBreakOnLog) != !bOldGlobalBreakValue;
		bOldGlobalBreakValue = GlobalVerbosity.DebugBreakOnLog;
		for (TPair<FName, FLogCategoryBase*> It : ReverseAssociations)
		{
			FLogCategoryBase* Verb = It.Value;
			ELogVerbosity::Type NewVerbosity = Verb->Verbosity;
			checkSlow(!(NewVerbosity & ELogVerbosity::BreakOnLog)); // this bit is factored out of this variable, always

			if (bVerbosityGoingDown)
			{
				NewVerbosity = FMath::Min<ELogVerbosity::Type>(GlobalVerbosity.Verbosity, Verb->Verbosity);
			}
			if (bVerbosityGoingUp)
			{
				NewVerbosity = FMath::Max<ELogVerbosity::Type>(GlobalVerbosity.Verbosity, Verb->Verbosity);
				NewVerbosity = FMath::Min<ELogVerbosity::Type>(Verb->CompileTimeVerbosity, NewVerbosity);
			}
			// store off the last non-zero one for toggle
			if (NewVerbosity)
			{
				// currently on, store this in the toggle for future use
				ToggleAssociations.Add(It.Key, NewVerbosity);
			}
			Verb->Verbosity = NewVerbosity;
			if (bForceBreak)
			{
				Verb->DebugBreakOnLog = GlobalVerbosity.DebugBreakOnLog;
			}
			checkSlow(!(Verb->Verbosity & ELogVerbosity::BreakOnLog)); // this bit is factored out of this variable, always
		}
	}

	/** 
	 * Called twice typically. Once when a log category is constructed, and then once after we have processed the command line.
	 * The second call is needed to make sure the default is set correctly when it is changed on the command line or config file.
	 **/
	void SetupSuppress(FLogCategoryBase* Destination, FName NameFName)
	{
		// now maybe this was set at boot, in which case we override what it had
		uint8* Boot = BootAssociations.Find(NameFName);
		if (Boot)
		{
			Destination->DefaultVerbosity = *Boot;
			Destination->ResetFromDefault();
		}
		else
		{
			// see if we had a boot global override
			static FName NAME_BootGlobal("BootGlobal");
			Boot = BootAssociations.Find(NAME_BootGlobal);
			if (Boot)
			{
				Destination->DefaultVerbosity = FMath::Min<uint8>(*Boot, Destination->GetCompileTimeVerbosity());
				Destination->ResetFromDefault();
			}
		}
		// store off the last non-zero one for toggle
		checkSlow(!(Destination->Verbosity & ELogVerbosity::BreakOnLog)); // this bit is factored out of this variable, always
		if (Destination->Verbosity)
		{
			// currently on, store this in the pending and clear it
			ToggleAssociations.Add(NameFName, Destination->Verbosity);
		}
	}
	
	virtual void AssociateSuppress(FLogCategoryBase* Destination)
	{
		check(Destination);
		if (bInitialized)
		{
			AssociateSuppressImpl(Destination);
		}
		else
		{
			PendingAssociations.Add(Destination);
		}
	}

	void AssociateSuppressImpl(FLogCategoryBase* Destination)
	{
		FName NameFName(Destination->CategoryName);
		bool bFoundExisting = false;
		for (TMultiMap<FName, FLogCategoryBase*>::TKeyIterator It(ReverseAssociations, NameFName); It; ++It)
		{
			if (It.Value() == Destination)
			{
				if (!IsReloadActive())
				{
					UE_LOGF(LogHAL, Fatal,"Log suppression category %ls was somehow declared twice with the same data.", *NameFName.ToString());
				}
			}
			// if it is registered, it better be the same
			if (It.Value()->CompileTimeVerbosity != Destination->CompileTimeVerbosity)
			{
				UE_LOGF(LogHAL, Fatal,"Log suppression category %ls is defined multiple times with different compile time verbosity.", *NameFName.ToString());
			}
			// we take whatever the existing one has to keep them in sync always
			checkSlow(!(It.Value()->Verbosity & ELogVerbosity::BreakOnLog)); // this bit is factored out of this variable, always
			Destination->Verbosity = It.Value()->Verbosity;
			Destination->DebugBreakOnLog = It.Value()->DebugBreakOnLog;
			Destination->DefaultVerbosity = It.Value()->DefaultVerbosity;
			bFoundExisting = true;
		}
		ReverseAssociations.Add(NameFName, Destination);
		if (bFoundExisting)
		{
			return; // in no case is there anything more to do...we want to match the other ones
		}
		SetupSuppress(Destination, NameFName); // this might be done again later if this is being set up before appInit is called
	}

	virtual void DisassociateSuppress(FLogCategoryBase* Destination)
	{
		int32 PendingIndex = PendingAssociations.FindLast(Destination);
		if (PendingIndex != INDEX_NONE)
		{
			PendingAssociations.RemoveAt(PendingIndex);
		}
		else
		{
			verify(ReverseAssociations.Remove(Destination->GetCategoryName(), Destination) == 1);
		}
	}

	void OnConfigSectionsChanged(const FString& IniFilename, const TSet<FString>& SectionNames)
	{
		if (IniFilename == GEngineIni && SectionNames.Contains(TEXT("Core.Log")))
		{
			ProcessConfigAndCommandLine();
		}
	}

	virtual void ProcessConfigAndCommandLine()
	{
		ReverseAssociations.Reserve(PendingAssociations.Num());
		for (FLogCategoryBase* Category : PendingAssociations)
		{
			AssociateSuppressImpl(Category);
		}
		PendingAssociations.Empty();

		// first we do the config values
		const FConfigSection* RefTypes = GConfig->GetSection(TEXT("Core.Log"), false, GEngineIni);
		if (RefTypes != NULL)
		{
			for( FConfigSectionMap::TConstIterator It(*RefTypes); It; ++It )
			{
				ProcessCmdString(It.Key().ToString() + TEXT(" ") + It.Value().GetValue(), true);
			}
		}
#if !UE_BUILD_SHIPPING
		// and the command line overrides the config values
		FString CmdLine(FCommandLine::Get());
		FString LogCmds("-LogCmds=");
		int32 IndexOfEnv = CmdLine.Find(TEXT("-EnvAfterHere"));
		if (IndexOfEnv != INDEX_NONE)
		{
			// if we have env variable stuff set on the command line, we want to process that FIRST
			FString CmdLineEnv = CmdLine.Mid(IndexOfEnv);
			while (1)
			{
				FString Cmds;
				if (!FParse::Value(*CmdLineEnv, *LogCmds, Cmds, false))
				{
					break;
				}
				ProcessCmdString(Cmds, true);
				// remove this command so that we can try for other ones...for example one on the command line and one coming from env vars
				int32 Index = CmdLineEnv.Find(*LogCmds);
				ensure(Index >= 0);
				if (Index == INDEX_NONE)
				{
					break;
				}
				CmdLineEnv.MidInline(Index + LogCmds.Len(), MAX_int32, EAllowShrinking::No);
			}
			// now strip off the environment arg part
			CmdLine.MidInline(0, IndexOfEnv, EAllowShrinking::No);
		}
		while (1)
		{
			FString Cmds;
			if (!FParse::Value(*CmdLine, *LogCmds, Cmds, false))
			{
				break;
			}
			ProcessCmdString(Cmds, true);
			// remove this command so that we can try for other ones...for example one on the command line and one coming from env vars
			int32 Index = CmdLine.Find(*LogCmds);
			ensure(Index >= 0);
			if (Index == INDEX_NONE)
			{
				break;
			}
			CmdLine.MidInline(Index + LogCmds.Len(), MAX_int32, EAllowShrinking::No);
		}
#endif // !UE_BUILD_SHIPPING

		// and then the compiled in defaults are overridden with those
		for (TMultiMap<FName, FLogCategoryBase*>::TIterator It(ReverseAssociations); It; ++It)
		{
			SetupSuppress(It.Value(), It.Key());
		}

		if (!bInitialized)
		{
			FCoreDelegates::TSOnConfigSectionsChanged().AddLambda([this](const FString& IniFilename, const TSet<FString>& SectionNames)
			{
				if (auto* LogSuppressionImplementation = static_cast<FLogSuppressionImplementation*>(FLogSuppressionInterface::TryGet()))
				{
					LogSuppressionImplementation->OnConfigSectionsChanged(IniFilename, SectionNames);
				}
				else
				{
					FCoreDelegates::TSOnConfigSectionsChanged().RemoveAll(this);
				}
			});
		}

		bInitialized = true;
	}

	/** Console commands, see embeded usage statement **/
	virtual bool Exec_Runtime( UWorld* Inworld, const TCHAR* Cmd, FOutputDevice& Ar )
	{
		if(FParse::Command(&Cmd,TEXT("LOG")))
		{
			if(FParse::Command(&Cmd,TEXT("LIST"))) // if they didn't use the list command, we will show usage
			{
				TArray<FLogCategoryPtrs> Found;

				FString Cat(FParse::Token(Cmd, 0));
				for (TPair<FName, FLogCategoryBase*> It : ReverseAssociations)
				{
					FLogCategoryBase* Verb = It.Value;
					FString Name = It.Key.ToString();
					if (!Cat.Len() || Name.Contains(Cat) )
					{
						Found.Add(FLogCategoryPtrs(Name, ELogVerbosity::Type(Verb->Verbosity), Verb->DebugBreakOnLog));
					}
				}

				Found.Sort();

				for (TArray<FLogCategoryPtrs>::TConstIterator It = Found.CreateConstIterator(); It; ++It)
				{
					Ar.Logf(TEXT("%-40s  %-12s  %s"), *It->Name, ToString(It->Verbosity), It->Postfix ? TEXT(" - DebugBreak") : TEXT(""));
				}

				return true;
			}
			else if (Cmd[0] != '.')
			{
				FString Rest(Cmd);
				Rest.TrimStartInline();
				if (Rest.Len())
				{
					if (ProcessLogOnly(Rest, Ar))
					{
						return true;
					}

					TMap<FName, uint8> OldValues;
					OldValues.Reserve(ReverseAssociations.Num());
					for (TPair<FName, FLogCategoryBase*> Pair : ReverseAssociations)
					{
						FLogCategoryBase* Verb = Pair.Value;
						FName Name = Pair.Key;
						OldValues.Add(Name, Verb->Verbosity);
					}
					ProcessCmdString(Rest);
					for (TPair<FName, FLogCategoryBase*> Pair : ReverseAssociations)
					{
						FLogCategoryBase* Verb = Pair.Value;
						FName Name = Pair.Key;
						uint8 OldValue = OldValues.FindRef(Name);
						if (Verb->Verbosity != OldValue)
						{
							Ar.Logf(TEXT("%-40s  %-12s  %s"), *Name.ToString(), ToString(ELogVerbosity::Type(Verb->Verbosity)), Verb->DebugBreakOnLog ? TEXT(" - DebugBreak") : TEXT(""));
						}
					}
				}
				else
				{
					Ar.Logf( TEXT("------- Log conventions") );
					Ar.Logf( TEXT("[cat]   = a category for the command to operate on, or 'global' for all categories.") );
					Ar.Logf( TEXT("[level] = verbosity level, one of: none, error, warning, display, log, verbose, all, default") );
					Ar.Logf( TEXT("At boot time, compiled in default is overridden by ini files setting, which is overridden by command line") );
					Ar.Logf( TEXT("------- Log console command usage") );
					Ar.Logf( TEXT("Log list            - list all log categories") );
					Ar.Logf( TEXT("Log list [string]   - list all log categories containing a substring") );
					Ar.Logf( TEXT("Log reset           - reset all log categories to their boot-time default") );
					Ar.Logf( TEXT("Log [cat]           - toggle the display of the category [cat]") );
					Ar.Logf( TEXT("Log [cat] off       - disable display of the category [cat]") );
					Ar.Logf( TEXT("Log [cat] on        - resume display of the category [cat]") );
					Ar.Logf( TEXT("Log [cat] only      - enables [cat] and disables all other categories"));
					Ar.Logf( TEXT("Log [cat] [level]   - set the verbosity level of the category [cat]") );
					Ar.Logf( TEXT("Log [cat] break     - toggle the debug break on display of the category [cat]") );
					Ar.Logf( TEXT("------- Log command line") );
					Ar.Logf( TEXT("-LogCmds=\"[arguments],[arguments]...\"           - applies a list of console commands at boot time") );
					Ar.Logf( TEXT("-LogCmds=\"foo verbose, bar off\"         - turns on the foo category and turns off the bar category") );					
					Ar.Logf( TEXT("------- Environment variables") );
					Ar.Logf( TEXT("Any command line option can be set via the environment variable UE-CmdLineArgs") );
					Ar.Logf( TEXT("set UE-CmdLineArgs=\"-LogCmds=foo verbose breakon, bar off\"") );
					Ar.Logf( TEXT("------- Config file") );
					Ar.Logf( TEXT("[Core.Log]") );
					Ar.Logf( TEXT("global=[default verbosity for things not listed later]") );					
					Ar.Logf( TEXT("[cat]=[level]") );					
					Ar.Logf( TEXT("foo=verbose break") );					
				}
				return true;
			}
		}
		return false;
	}
};

FLogSuppressionInterface& FLogSuppressionInterface::Get()
{
	return TLazySingleton<FLogSuppressionImplementation>::Get();
}

FLogSuppressionInterface* FLogSuppressionInterface::TryGet()
{
	return TLazySingleton<FLogSuppressionImplementation>::TryGet();
}

void FLogSuppressionInterface::TearDown()
{
	TLazySingleton<FLogSuppressionImplementation>::TearDown();
}
