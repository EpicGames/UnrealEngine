// Copyright Epic Games, Inc. All Rights Reserved.
#include "Logging/LogTrace.h"

#if LOGTRACE_ENABLED

#include "HAL/PlatformTime.h"
#include "HAL/PlatformTLS.h"
#include "Misc/ScopeLock.h"
#include "Misc/VarArgs.h"
#include "Templates/Function.h"
#include "Trace/Trace.inl"

UE_TRACE_CHANNEL_DEFINE(LogChannel,
	"Captures engine UE_LOG output (category, verbosity, file/line, formatted message); "
	"shown in the Log view in Unreal Insights.")

UE_TRACE_EVENT_BEGIN(Logging, LogCategory, NoSync|Important)
	UE_TRACE_EVENT_FIELD(const void*, CategoryPointer)
	UE_TRACE_EVENT_FIELD(uint8, DefaultVerbosity)
	UE_TRACE_EVENT_FIELD(UE::Trace::AnsiString, Name)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Logging, LogMessageSpec, NoSync|Important)
	UE_TRACE_EVENT_FIELD(const void*, LogPoint)
	UE_TRACE_EVENT_FIELD(const void*, CategoryPointer)
	UE_TRACE_EVENT_FIELD(int32, Line)
	UE_TRACE_EVENT_FIELD(uint8, Verbosity)
	UE_TRACE_EVENT_FIELD(UE::Trace::AnsiString, FileName)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, FormatString)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Logging, LogMessage, NoSync)
	UE_TRACE_EVENT_FIELD(const void*, LogPoint)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(uint8[], FormatArgs)
UE_TRACE_EVENT_END()

void FLogTrace::OutputLogCategory(const FLogCategoryBase* Category, const TCHAR* Name, ELogVerbosity::Type DefaultVerbosity)
{
	uint16 NameLen = uint16(FCString::Strlen(Name));
	UE_TRACE_LOG(Logging, LogCategory, LogChannel, NameLen * sizeof(ANSICHAR))
		<< LogCategory.CategoryPointer(Category)
		<< LogCategory.DefaultVerbosity(DefaultVerbosity)
		<< LogCategory.Name(Name, NameLen);
}

void FLogTrace::OutputLogMessageSpec(const void* LogPoint, const FLogCategoryBase* Category, ELogVerbosity::Type Verbosity, const ANSICHAR* File, int32 Line, const TCHAR* Format)
{
	uint16 FileNameLen = uint16(strlen(File));
	uint16 FormatStringLen = uint16(FCString::Strlen(Format));
	uint32 DataSize = (FileNameLen * sizeof(ANSICHAR)) + (FormatStringLen * sizeof(TCHAR));
	UE_TRACE_LOG(Logging, LogMessageSpec, LogChannel, DataSize)
		<< LogMessageSpec.LogPoint(LogPoint)
		<< LogMessageSpec.CategoryPointer(Category)
		<< LogMessageSpec.Line(Line)
		<< LogMessageSpec.Verbosity(Verbosity)
		<< LogMessageSpec.FileName(File, FileNameLen)
		<< LogMessageSpec.FormatString(Format, FormatStringLen);
}

void FLogTrace::OutputLogMessageInternal(const void* LogPoint, int32 EncodedFormatArgsSize, const uint8* EncodedFormatArgs)
{
	UE_TRACE_LOG(Logging, LogMessage, LogChannel)
		<< LogMessage.LogPoint(LogPoint)
		<< LogMessage.Cycle(FPlatformTime::Cycles64())
		<< LogMessage.FormatArgs(EncodedFormatArgs, EncodedFormatArgsSize);
}

#if LOGTRACE_RUNTIME_FORMATTING_ENABLED
static FCriticalSection* GetLogTraceStaticBufferGuard()
{
	static FCriticalSection CS;
	return &CS;
}

static TCHAR LogTraceStaticBuffer[8192];

void FLogTrace::OutputLogMessageSimple(const void* LogPoint, const TCHAR* Fmt, ...)
{
	FScopeLock MsgLock(GetLogTraceStaticBufferGuard());
	GET_TYPED_VARARGS(TCHAR, LogTraceStaticBuffer, UE_ARRAY_COUNT(LogTraceStaticBuffer), UE_ARRAY_COUNT(LogTraceStaticBuffer) - 1, Fmt, Fmt);
	FLogTrace::OutputLogMessage(LogPoint, LogTraceStaticBuffer);
}
#endif // LOGTRACE_RUNTIME_FORMATTING_ENABLED

#endif // LOGTRACE_ENABLED
