// Copyright Epic Games, Inc. All Rights Reserved.

#include "Logging/LogVerbosity.h"
#include "Containers/UnrealString.h"

const TCHAR* ToString(ELogVerbosity::Type Verbosity)
{
	switch (Verbosity & ELogVerbosity::VerbosityMask)
	{
	case ELogVerbosity::NoLogging:
		return TEXT("NoLogging");
	case ELogVerbosity::Fatal:
		return TEXT("Fatal");
	case ELogVerbosity::Error:
		return TEXT("Error");
	case ELogVerbosity::Warning:
		return TEXT("Warning");
	case ELogVerbosity::Display:
		return TEXT("Display");
	case ELogVerbosity::Log:
		return TEXT("Log");
	case ELogVerbosity::Verbose:
		return TEXT("Verbose");
	case ELogVerbosity::VeryVerbose:
		return TEXT("VeryVerbose");
	}
	return TEXT("UnknownVerbosity");
}

CORE_API ELogVerbosity::Type ParseLogVerbosityFromString(const FString& VerbosityString)
{
	if (VerbosityString == TEXT("NoLogging"))
	{
		return ELogVerbosity::NoLogging;
	}
	else if (VerbosityString == TEXT("Fatal"))
	{
		return ELogVerbosity::Fatal;
	}
	else if (VerbosityString == TEXT("Error"))
	{
		return ELogVerbosity::Error;
	}
	else if (VerbosityString == TEXT("Warning"))
	{
		return ELogVerbosity::Warning;
	}
	else if (VerbosityString == TEXT("Display"))
	{
		return ELogVerbosity::Display;
	}
	else if (VerbosityString == TEXT("Log"))
	{
		return ELogVerbosity::Log;
	}
	else if (VerbosityString == TEXT("Verbose"))
	{
		return ELogVerbosity::Verbose;
	}
	else if (VerbosityString == TEXT("VeryVerbose"))
	{
		return ELogVerbosity::VeryVerbose;
	}
	else
	{
		// An unknown value is treated as log
		return ELogVerbosity::Log;
	}
}