// Copyright Epic Games, Inc. All Rights Reserved.

#include "Logging/MessageLog.h"

#include "HAL/ConsoleManager.h"
#include "Internationalization/Internationalization.h"
#include "Logging/IMessageLog.h"

FMessageLog::FGetLog FMessageLog::GetLog;

FMessageLog::FMessageSelectionChanged FMessageLog::MessageSelectionChanged;
FMessageLog::FMultiSelectActorBegin FMessageLog::MultiSelectActorBegin;
FMessageLog::FMultiSelectActorEnd FMessageLog::MultiSelectActorEnd;

#define LOCTEXT_NAMESPACE "MessageLog"

LLM_DEFINE_TAG(EngineMisc_MessageLog);

namespace UE::MessageLog::Private
{
	bool LogInfoMessagesAtDisplayVerbosity = 0;
	FAutoConsoleVariableRef CVarLogInfoMessagesAtDisplayVerbosity(
		TEXT("MessageLog.InfoMessagesAtDisplayVerbosity"),
		LogInfoMessagesAtDisplayVerbosity,
		TEXT("MessageLog messages at level 'info' will be mirrored to the log at 'Display' verbosity rather than 'Log' verbosity."),
		ECVF_Default);
	
	using FOverrideStackMap = TSortedMap<FName, TArray<const FMessageLogScopedOverride*>, FDefaultAllocator, FNameFastLess>;

	/** Get the internal stack of FMessageLogScopedOverride's */
	FOverrideStackMap& GetOverrideStack()
	{
		static FOverrideStackMap OverrideStack;
		return OverrideStack;
	}

	/** Get the current FMessageLogScopedOverride for the given log name, if any */
	const FMessageLogScopedOverride* GetOverride(const FName LogName)
	{
		if (TArray<const FMessageLogScopedOverride*>* NamedOverrideStack = GetOverrideStack().Find(LogName);
			NamedOverrideStack && NamedOverrideStack->Num() > 0)
		{
			return NamedOverrideStack->Top();
		}
		return nullptr;
	}
}

class FBasicMessageLog : public IMessageLog, public TSharedFromThis<FBasicMessageLog>
{
public:
	FBasicMessageLog( const FName& InLogName )
		: LogName( InLogName )
	{
	}

	/** Begin IMessageLog interface */
	virtual void AddMessage( const TSharedRef<FTokenizedMessage>& NewMessage, bool bMirrorToOutputLog ) override
	{
		AddMessageInternal(NewMessage, bMirrorToOutputLog);
	}

	virtual void AddMessages( const TArray< TSharedRef<FTokenizedMessage> >& NewMessages, bool bMirrorToOutputLog ) override
	{
		for(TArray< TSharedRef<FTokenizedMessage> >::TConstIterator It = NewMessages.CreateConstIterator(); It; It++)
		{
			AddMessageInternal(*It, bMirrorToOutputLog);
		}
	}

	virtual void NewPage( const FText& Title ) override
	{
		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("PageTitle"), Title);
		FMsg::Logf(__FILE__, __LINE__, LogName, ELogVerbosity::Log, TEXT("%s"), *FText::Format(LOCTEXT("BasicMessageLog_NewPage", "New Page: {PageTitle}"), Arguments).ToString());
	}

	virtual void SetCurrentPage( const FText& Title) override
	{
		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("PageTitle"), Title);
		FMsg::Logf(__FILE__, __LINE__, LogName, ELogVerbosity::Log, TEXT("%s"), *FText::Format(LOCTEXT("BasicMessageLog_SetCurrentPageTitle", "Set Current Page: {PageTitle}"), Arguments).ToString());
	}

	virtual void SetCurrentPage( const uint32 PageIndex) override
	{
		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("PageIndex"), PageIndex);
		FMsg::Logf(__FILE__, __LINE__, LogName, ELogVerbosity::Log, TEXT("%s"), *FText::Format(LOCTEXT("BasicMessageLog_SetCurrentPageIndex", "Set Current Page: {PageIndex}"), Arguments).ToString());
	}

	virtual void NotifyIfAnyMessages( const FText& Message, EMessageSeverity::Type SeverityFilter, bool bForce ) override
	{
		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("Message"), Message);
		FMsg::Logf(__FILE__, __LINE__, LogName, ELogVerbosity::Log, TEXT("%s"), *FText::Format(LOCTEXT("BasicMessageLog_Notify", "Notify: {Message}"), Arguments).ToString());
	}

	virtual void Open() override
	{
		FMsg::Logf(__FILE__, __LINE__, LogName, ELogVerbosity::Log, TEXT("%s"), *LOCTEXT("BasicMessageLog_Open", "Open Log").ToString());
	}

	virtual int32 NumMessages( EMessageSeverity::Type SeverityFilter ) override
	{
		return 0;
	}
	/** End IMessageLog interface */

private:
	void AddMessageInternal( const TSharedRef<FTokenizedMessage>& Message, bool bMirrorToOutputLog )
	{
		if (bMirrorToOutputLog)
		{
			const TCHAR* const LogColor = FMessageLog::GetLogColor(Message->GetSeverity());
			if (LogColor)
			{
				SET_WARN_COLOR(LogColor);
			}
			FMsg::Logf(__FILE__, __LINE__, LogName, FMessageLog::GetLogVerbosity(Message->GetSeverity()), TEXT("%s"), *Message->ToText().ToString());
			CLEAR_WARN_COLOR();
		}
	}

private:
	/** The name of this log */
	FLogCategoryName LogName;
};

FMessageLog::FMessageLog( const FName& InLogName )
	: LogName(InLogName)
	, bSuppressLoggingToOutputLog(false)
{
	LLM_SCOPE_BYTAG(EngineMisc_MessageLog);
	
	if(GetLog.IsBound())
	{
		MessageLog = GetLog.Execute(InLogName);
	}
	else
	{
		MessageLog = MakeShareable( new FBasicMessageLog(InLogName) );
	}
}

FMessageLog::~FMessageLog()
{
	LLM_SCOPE_BYTAG(EngineMisc_MessageLog);
	
	Flush();
}

const TSharedRef<FTokenizedMessage>& FMessageLog::AddMessage( const TSharedRef<FTokenizedMessage>& InMessage )
{
	LLM_SCOPE_BYTAG(EngineMisc_MessageLog);
	
	Messages.Add(InMessage);

	return InMessage;
}

void FMessageLog::AddMessages( const TArray< TSharedRef<FTokenizedMessage> >& InMessages )
{
	LLM_SCOPE_BYTAG(EngineMisc_MessageLog);
	
	Messages.Append(InMessages);
}

TSharedRef<FTokenizedMessage> FMessageLog::Message( EMessageSeverity::Type InSeverity, const FText& InMessage )
{
	LLM_SCOPE_BYTAG(EngineMisc_MessageLog);
	
	TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create(InSeverity, InMessage);
	Messages.Add(Message);
	return Message;
}

TSharedRef<FTokenizedMessage> FMessageLog::Error( const FText& InMessage )
{
	LLM_SCOPE_BYTAG(EngineMisc_MessageLog);
	
	TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create(EMessageSeverity::Error, InMessage);
	Messages.Add(Message);
	return Message;
}

TSharedRef<FTokenizedMessage> FMessageLog::PerformanceWarning( const FText& InMessage )
{
	LLM_SCOPE_BYTAG(EngineMisc_MessageLog);
	
	TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create(EMessageSeverity::PerformanceWarning, InMessage);
	Messages.Add(Message);
	return Message;
}

TSharedRef<FTokenizedMessage> FMessageLog::Warning( const FText& InMessage )
{
	LLM_SCOPE_BYTAG(EngineMisc_MessageLog);
	
	TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create(EMessageSeverity::Warning, InMessage);
	Messages.Add(Message);
	return Message;
}

TSharedRef<FTokenizedMessage> FMessageLog::Info( const FText& InMessage )
{
	LLM_SCOPE_BYTAG(EngineMisc_MessageLog);
	
	TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create(EMessageSeverity::Info, InMessage);
	Messages.Add(Message);
	return Message;
}

int32 FMessageLog::NumMessages( EMessageSeverity::Type InSeverityFilter )
{
	LLM_SCOPE_BYTAG(EngineMisc_MessageLog);
	
	Flush();
	return MessageLog->NumMessages(InSeverityFilter);
}

void FMessageLog::Open( EMessageSeverity::Type InSeverityFilter, bool bOpenEvenIfEmpty )
{
	LLM_SCOPE_BYTAG(EngineMisc_MessageLog);
	
	Flush();
	if(bOpenEvenIfEmpty)
	{
		MessageLog->Open();
	}
	else if(MessageLog->NumMessages(InSeverityFilter) > 0)
	{
		MessageLog->Open();
	}
}

void FMessageLog::Notify( const FText& InMessage, EMessageSeverity::Type InSeverityFilter, bool bForce )
{
	LLM_SCOPE_BYTAG(EngineMisc_MessageLog);
	
	Flush();
	MessageLog->NotifyIfAnyMessages(InMessage, InSeverityFilter, bForce);
}

void FMessageLog::NewPage( const FText& InLabel )
{
	LLM_SCOPE_BYTAG(EngineMisc_MessageLog);
	
	Flush();
	MessageLog->NewPage(InLabel);
}

void FMessageLog::SetCurrentPage(const FText& InLabel)
{
	LLM_SCOPE_BYTAG(EngineMisc_MessageLog);
	
	Flush();
	MessageLog->SetCurrentPage(InLabel);
}

FMessageLog& FMessageLog::SuppressLoggingToOutputLog(bool bShouldSuppress)
{
	bSuppressLoggingToOutputLog = bShouldSuppress;
	return *this;
}

void FMessageLog::Flush()
{
	if(Messages.Num() > 0)
	{
		bool bMirrorToOutputLog = !bSuppressLoggingToOutputLog;

		// We defer querying the FMessageLogScopedOverride until Flush to minimize the number of map lookups while adding messages
		// Though this does mean that messages added during an overload (and not flushed) may log with the wrong severity override
		if (const FMessageLogScopedOverride* LogOverride = IsInGameThread() ? UE::MessageLog::Private::GetOverride(LogName) : nullptr)
		{
			bMirrorToOutputLog = !LogOverride->bSuppressLoggingToOutputLog.Get(bSuppressLoggingToOutputLog);

			if (LogOverride->MessageSeverityRemapping.Num() > 0)
			{
				auto ResolveMessageSeverity = [LogOverride](const EMessageSeverity::Type BaseSeverity) -> EMessageSeverity::Type
				{
					EMessageSeverity::Type ResolvedSeverity = BaseSeverity;
					{
						TArray<EMessageSeverity::Type, TInlineAllocator<4>> CycleDetector;
						EMessageSeverity::Type DemotedSeverity = BaseSeverity;
						for (;;)
						{
							CycleDetector.Add(ResolvedSeverity);
							if (const EMessageSeverity::Type* MessageSeverityOverride = LogOverride->MessageSeverityRemapping.Find(ResolvedSeverity))
							{
								if (CycleDetector.Contains(*MessageSeverityOverride))
								{
									// Remapping cycle!
									// Since we can't tell whether the intent was to promote or demote this severity, 
									// we pessimistically assume the intent was to demote and use the most demoted value
									ResolvedSeverity = DemotedSeverity;
									break;
								}
								ResolvedSeverity = *MessageSeverityOverride;
								DemotedSeverity = FMath::Max(DemotedSeverity, ResolvedSeverity); // Bigger numbers are lower severity
								continue;
							}
							break;
						}
					}
					return ResolvedSeverity;
				};

				for (TSharedRef<FTokenizedMessage>& Message : Messages)
				{
					const EMessageSeverity::Type ResolvedSeverity = ResolveMessageSeverity(Message->GetSeverity());
					if (ResolvedSeverity != Message->GetSeverity())
					{
						// Need to clone the FTokenizedMessage as the severity value may be shared with other usages of this message
						Message = Message->Clone();
						Message->SetSeverity(ResolvedSeverity);
					}
				}
			}
		}

		MessageLog->AddMessages(Messages, bMirrorToOutputLog);
		Messages.Empty();
	}
}

ELogVerbosity::Type FMessageLog::GetLogVerbosity( EMessageSeverity::Type InSeverity )
{
	switch(InSeverity)
	{
	case EMessageSeverity::Error:
		return ELogVerbosity::Error;
	case EMessageSeverity::PerformanceWarning:
	case EMessageSeverity::Warning:
		return ELogVerbosity::Warning;
	case EMessageSeverity::Info:
		return UE::MessageLog::Private::LogInfoMessagesAtDisplayVerbosity ? ELogVerbosity::Display : ELogVerbosity::Log;
	default:
		return ELogVerbosity::Log;
	}
}

const TCHAR* const FMessageLog::GetLogColor( EMessageSeverity::Type InSeverity )
{
#if !PLATFORM_SUPPORTS_COLORIZED_OUTPUT_DEVICE
	return NULL;
#else
	switch(InSeverity)
	{
	case EMessageSeverity::Error:
		return OutputDeviceColor::COLOR_RED;
	case EMessageSeverity::PerformanceWarning:
	case EMessageSeverity::Warning:
		return OutputDeviceColor::COLOR_YELLOW;
	case EMessageSeverity::Info:
	default:
		return nullptr;
	}
#endif
}


FMessageLogScopedOverride::FMessageLogScopedOverride(const FName InLogName)
	: LogName(InLogName)
{
	LLM_SCOPE_BYTAG(EngineMisc_MessageLog);

	checkf(IsInGameThread(), TEXT("FMessageLogScopedOverride should only be pushed from the game-thread!"));
	TArray<const FMessageLogScopedOverride*>& NamedOverrideStack = UE::MessageLog::Private::GetOverrideStack().FindOrAdd(LogName);
	if (NamedOverrideStack.Num() > 0)
	{
		// Set our initial state to the top of the current stack, as that "inherits" the current override state
		const FMessageLogScopedOverride* CurrentOverride = NamedOverrideStack.Top();
		bSuppressLoggingToOutputLog = CurrentOverride->bSuppressLoggingToOutputLog;
		MessageSeverityRemapping = CurrentOverride->MessageSeverityRemapping;
	}
	NamedOverrideStack.Push(this);
}

FMessageLogScopedOverride::~FMessageLogScopedOverride()
{
	LLM_SCOPE_BYTAG(EngineMisc_MessageLog);

	checkf(IsInGameThread(), TEXT("FMessageLogScopedOverride should only be popped from the game-thread!"));
	if (TArray<const FMessageLogScopedOverride*>* NamedOverrideStack = UE::MessageLog::Private::GetOverrideStack().Find(LogName))
	{
		const FMessageLogScopedOverride* This = NamedOverrideStack->Pop();
		checkf(this == This, TEXT("FMessageLogScopedOverride popped an unexpected override off the stack! Out of order destruction of FMessageLogScopedOverride!"));
	}
	else
	{
		checkf(false, TEXT("FMessageLogScopedOverride popped an unexpected log name off the stack! Out of order destruction of FMessageLogScopedOverride!"));
	}
}

FMessageLogScopedOverride& FMessageLogScopedOverride::SuppressLoggingToOutputLog(const bool bShouldSuppress)
{
	bSuppressLoggingToOutputLog = bShouldSuppress;
	return *this;
}

FMessageLogScopedOverride& FMessageLogScopedOverride::RemapMessageSeverity(const EMessageSeverity::Type SrcSeverity, const EMessageSeverity::Type DestSeverity)
{
	LLM_SCOPE_BYTAG(EngineMisc_MessageLog);

	MessageSeverityRemapping.Add(SrcSeverity, DestSeverity);
	return *this;
}

#undef LOCTEXT_NAMESPACE
