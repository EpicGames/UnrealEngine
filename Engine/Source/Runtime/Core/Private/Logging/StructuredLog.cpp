// Copyright Epic Games, Inc. All Rights Reserved.

#include "Logging/StructuredLog.h"

#include "Algo/Accumulate.h"
#include "Algo/AllOf.h"
#include "Algo/Find.h"
#include "Algo/NoneOf.h"
#include "Async/Mutex.h"
#include "Async/TransactionallySafeMutex.h"
#include "Async/UniqueLock.h"
#include "AutoRTFM.h"
#include "Containers/AnsiString.h"
#include "Containers/SparseArray.h"
#include "Containers/StringConv.h"
#include "Containers/StringView.h"
#include "CoreGlobals.h"
#include "HAL/LowLevelMemTracker.h"
#include "HAL/PlatformMallocCrash.h"
#include "HAL/PlatformMath.h"
#include "HAL/PlatformTime.h"
#include "Internationalization/Text.h"
#include "Logging/LogTrace.h"
#include "Logging/StructuredLogFormat.h"
#include "Misc/AsciiSet.h"
#include "Misc/DateTime.h"
#include "Misc/FeedbackContext.h"
#include "Misc/OutputDevice.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Misc/ScopeExit.h"
#include "Misc/StringBuilder.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "Serialization/CompactBinarySerialization.h"
#include "Serialization/CompactBinaryValue.h"
#include "Serialization/CompactBinaryWriter.h"
#include "Serialization/VarInt.h"
#include "Stats/Stats.h"
#include "String/LexFromString.h"
#include "String/Numeric.h"
#include "String/Split.h"
#include "Templates/Function.h"
#include <cstdarg>

CSV_DECLARE_CATEGORY_EXTERN(FMsgLogf);

void StaticFailDebug (const TCHAR* Error, const ANSICHAR* File, int32 Line, void* ProgramCounter, const TCHAR* Message);
void StaticFailDebugV(const TCHAR* Error, const ANSICHAR* File, int32 Line, void* ProgramCounter, const TCHAR* DescriptionFormat, va_list DescriptionArgs);
void StaticFailDebugV(const TCHAR* Error, const ANSICHAR* File, int32 Line, void* ProgramCounter, const ANSICHAR* DescriptionFormat, va_list DescriptionArgs);
void StaticFailDebugV(const TCHAR* Error, const ANSICHAR* File, int32 Line, void* ProgramCounter, const UTF8CHAR* DescriptionFormat, va_list DescriptionArgs);

namespace UE::Serialization::Private
{
template <typename CharType>
void AppendQuotedJsonString(TStringBuilderBase<CharType>& Builder, FUtf8StringView Value);
} // UE::Serialization::Private

namespace UE::Logging::Private
{

// Temporarily provided to toggle back to the previous behavior from code.
CORE_API bool GConvertBasicLogToLogRecord = true;

// Experimental feature to prepend log context to the log message during formatting.
CORE_API bool GPrependLogContextToLogMessage = false;

static constexpr ANSICHAR GFieldPathDelimiter = '/';
static constexpr FAsciiSet GValidLogFieldName("0123456789ABCDEFGHIJKLMNOPQRSTUVWXYZabcdefghijklmnopqrstuvwxyz_");
static constexpr FAsciiSet GValidLogFieldPath = GValidLogFieldName | FAsciiSet({GFieldPathDelimiter, '[', ']', '\0'});

static constexpr FAnsiStringView GLogContextsFieldName = ANSITEXTVIEW("$Contexts");

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

struct FLogTemplateOp
{
	enum EOpCode : int32 { OpEnd, OpSkip, OpText, OpName, OpPath, OpIndex, OpLocalized, OpCount };

	static constexpr int32 ValueShift = 3;
	static_assert(OpCount <= (1 << ValueShift));

	EOpCode Code = OpEnd;
	int32 Value = 0;

	inline int32 GetSkipSize() const
	{
		return Code == OpIndex || Code == OpLocalized ? 0 : Value;
	}

	static inline FLogTemplateOp Load(const uint8*& Data);
	static inline uint32 SaveSize(const FLogTemplateOp& Op) { return MeasureVarUInt(Encode(Op)); }
	static inline void Save(const FLogTemplateOp& Op, uint8*& Data);
	static constexpr uint64 Encode(const FLogTemplateOp& Op) { return uint64(Op.Code) | (uint64(Op.Value) << ValueShift); }
	static constexpr FLogTemplateOp Decode(uint64 Value) { return {EOpCode(Value & ((1 << ValueShift) - 1)), int32(Value >> ValueShift)}; }
};

static_assert(FLogTemplateOp::Decode(FLogTemplateOp::Encode({.Value = 123})).Value == 123);
static_assert(FLogTemplateOp::Decode(FLogTemplateOp::Encode({.Value = -123})).Value == -123);

inline FLogTemplateOp FLogTemplateOp::Load(const uint8*& Data)
{
	uint32 ByteCount = 0;
	ON_SCOPE_EXIT { Data += ByteCount; };
	return Decode(ReadVarUInt(Data, ByteCount));
}

inline void FLogTemplateOp::Save(const FLogTemplateOp& Op, uint8*& Data)
{
	Data += WriteVarUInt(Encode(Op), Data);
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

template <typename CharType>
struct TLogFieldValueConstants;

template <>
struct TLogFieldValueConstants<UTF8CHAR>
{
	static inline const FAnsiStringView Null = ANSITEXTVIEW("null");
	static inline const FAnsiStringView True = ANSITEXTVIEW("true");
	static inline const FAnsiStringView False = ANSITEXTVIEW("false");
};

template <>
struct TLogFieldValueConstants<WIDECHAR>
{
	static inline const FWideStringView Null = WIDETEXTVIEW("null");
	static inline const FWideStringView True = WIDETEXTVIEW("true");
	static inline const FWideStringView False = WIDETEXTVIEW("false");
};

template <typename CharType>
static void LogFieldValue(TStringBuilderBase<CharType>& Out, const FCbFieldView& Field)
{
	using FConstants = TLogFieldValueConstants<CharType>;
	switch (FCbValue Accessor = Field.GetValue(); Accessor.GetType())
	{
	case ECbFieldType::Null:
		Out.Append(FConstants::Null);
		break;
	case ECbFieldType::Object:
	case ECbFieldType::UniformObject:
		{
			FCbObjectView Object = Accessor.AsObjectView();

			// Use $text if present.
			if (FCbFieldView TextField = Object.FindViewIgnoreCase(ANSITEXTVIEW("$text")); TextField.IsString())
			{
				Out.Append(TextField.AsString());
				break;
			}

			// Use $format for formatting if present.
			if (FCbFieldView FormatField = Object.FindViewIgnoreCase(ANSITEXTVIEW("$format")); FormatField.IsString())
			{
				TUtf8StringBuilder<128> Format(InPlace, FormatField.AsString());
				FInlineLogTemplate Template(*Format, {.bAllowSubObjectReferences = true});
				Template.FormatTo(Out, Object.CreateViewIterator());
				break;
			}

			// Use $locformat/$locns/$loctext for localized formatting if present.
			FCbFieldView LocFormatField = Object.FindViewIgnoreCase(ANSITEXTVIEW("$locformat"));
			FCbFieldView LocNamespaceField = Object.FindViewIgnoreCase(ANSITEXTVIEW("$locns"));
			FCbFieldView LocKeyField = Object.FindViewIgnoreCase(ANSITEXTVIEW("$lockey"));
			if (LocFormatField.IsString() && LocNamespaceField.IsString() && LocKeyField.IsString())
			{
				TStringBuilder<32> Namespace(InPlace, LocNamespaceField.AsString());
				TStringBuilder<32> Key(InPlace, LocKeyField.AsString());
				TUtf8StringBuilder<128> Format(InPlace, LocFormatField.AsString());
				FInlineLogTemplate Template(*Namespace, *Key, *Format, {.bAllowSubObjectReferences = true});
				Template.FormatTo(Out, Object.CreateViewIterator());
				break;
			}

			// Write as JSON as a fallback.
			Out.AppendChar('{');
			bool bNeedsComma = false;
			for (FCbFieldView It : Field)
			{
				if (bNeedsComma)
				{
					Out.AppendChar(',').AppendChar(' ');
				}
				bNeedsComma = true;
				Serialization::Private::AppendQuotedJsonString(Out, It.GetName());
				Out.AppendChar(':').AppendChar(' ');
				LogFieldValue(Out, It);
			}
			Out.AppendChar('}');
		}
		break;
	case ECbFieldType::Array:
	case ECbFieldType::UniformArray:
		{
			Out.AppendChar('[');
			bool bNeedsComma = false;
			for (FCbFieldView It : Field)
			{
				if (bNeedsComma)
				{
					Out.AppendChar(',').AppendChar(' ');
				}
				bNeedsComma = true;
				LogFieldValue(Out, It);
			}
			Out.AppendChar(']');
		}
		break;
	case ECbFieldType::Binary:
		CompactBinaryToCompactJson(Field.RemoveName(), Out);
		break;
	case ECbFieldType::String:
		Out.Append(Accessor.AsString());
		break;
	case ECbFieldType::IntegerPositive:
		Out << Accessor.AsIntegerPositive();
		break;
	case ECbFieldType::IntegerNegative:
		Out << Accessor.AsIntegerNegative();
		break;
	case ECbFieldType::Float32:
	case ECbFieldType::Float64:
		CompactBinaryToCompactJson(Field.RemoveName(), Out);
		break;
	case ECbFieldType::BoolFalse:
		Out.Append(FConstants::False);
		break;
	case ECbFieldType::BoolTrue:
		Out.Append(FConstants::True);
		break;
	case ECbFieldType::ObjectAttachment:
	case ECbFieldType::BinaryAttachment:
		Out << Accessor.AsAttachment();
		break;
	case ECbFieldType::Hash:
		Out << Accessor.AsHash();
		break;
	case ECbFieldType::Uuid:
		Out << Accessor.AsUuid();
		break;
	case ECbFieldType::DateTime:
		Out << FDateTime(Accessor.AsDateTimeTicks()).ToIso8601();
		break;
	case ECbFieldType::TimeSpan:
	{
		const FTimespan Span(Accessor.AsTimeSpanTicks());
		if (Span.GetDays() == 0)
		{
			Out << Span.ToString(TEXT("%h:%m:%s.%n"));
		}
		else
		{
			Out << Span.ToString(TEXT("%d.%h:%m:%s.%n"));
		}
		break;
	}
	case ECbFieldType::ObjectId:
		Out << Accessor.AsObjectId();
		break;
	case ECbFieldType::CustomById:
	case ECbFieldType::CustomByName:
		CompactBinaryToCompactJson(Field.RemoveName(), Out);
		break;
	default:
		checkNoEntry();
		break;
	}
}

static void AddFieldValue(FFormatNamedArguments& Out, FUtf8StringView FieldPath, const FCbFieldView& Field)
{
	FString FieldName(FieldPath);

	switch (FCbValue Accessor = Field.GetValue(); Accessor.GetType())
	{
	case ECbFieldType::IntegerPositive:
		Out.Emplace(MoveTemp(FieldName), Accessor.AsIntegerPositive());
		return;
	case ECbFieldType::IntegerNegative:
		Out.Emplace(MoveTemp(FieldName), Accessor.AsIntegerNegative());
		return;
	case ECbFieldType::Float32:
		Out.Emplace(MoveTemp(FieldName), Accessor.AsFloat32());
		return;
	case ECbFieldType::Float64:
		Out.Emplace(MoveTemp(FieldName), Accessor.AsFloat64());
		return;
	default:
		break;
	}

	// Handle anything that falls through as text.
	TStringBuilder<128> Text;
	LogFieldValue(Text, Field);
	Out.Emplace(MoveTemp(FieldName), FText::FromString(FString(Text)));
}

template <typename FormatCharType>
class TFieldFinder
{
public:
	inline TFieldFinder(const FormatCharType* InFormat, const FCbFieldViewIterator& InFields)
		: Format(InFormat)
		, Fields(InFields)
	{
	}

	const FCbFieldView& Find(FUtf8StringView Name, int32 IndexHint = -1)
	{
		if (IndexHint >= 0)
		{
			for (; Index < IndexHint && It; ++Index, ++It)
			{
			}
			if (IndexHint < Index)
			{
				It = Fields;
				for (Index = 0; Index < IndexHint && It; ++Index, ++It);
			}
			if (IndexHint == Index && Name.Equals(It.GetName()))
			{
				return It;
			}
		}
		const int32 PrevIndex = Index;
		for (; It; ++Index, ++It)
		{
			if (Name.Equals(It.GetName()))
			{
				return It;
			}
		}
		It = Fields;
		for (Index = 0; Index < PrevIndex && It; ++Index, ++It)
		{
			if (Name.Equals(It.GetName()))
			{
				return It;
			}
		}
		checkf(false, TEXT("Log format requires field '%s' which was not provided. [[%s]]"),
			*WriteToString<32>(Name), StringCast<TCHAR>(Format).Get());
		return It;
	}

	FCbFieldView FindByPath(FUtf8StringView Path, int32 IndexHint = -1)
	{
		FUtf8StringView Name = Path;
		FUtf8StringView IndexDigits;
		bool bMore = String::SplitFirstChar(Path, UTF8CHAR(GFieldPathDelimiter), Name, Path);
		bool bArray = Name.EndsWith(']') && String::SplitLastChar(Name, UTF8CHAR('['), Name, IndexDigits);
		FCbFieldView Field = Find(Name, IndexHint);
		for (;;)
		{
			if (UNLIKELY(bArray))
			{
				IndexDigits.RemoveSuffix(1);
				checkf(String::IsNumericOnlyDigits(IndexDigits), TEXT("Log format has non-numeric array index '%s'. [[%s]]"),
					*WriteToString<32>(IndexDigits), StringCast<TCHAR>(Format).Get());
				FCbArrayView Array = Field.AsArrayView();
				checkf(!Field.HasError(), TEXT("Log format requires field '%s' but the corresponding field is not an array. [[%s]]"),
					*WriteToString<32>(Name, '[', IndexDigits, ']'), StringCast<TCHAR>(Format).Get());
				int32 ArrayIndex = 0;
				LexFromString(ArrayIndex, IndexDigits);
				checkf(ArrayIndex < Array.Num(), TEXT("Log format requires item at index %d in array '%s' but the array only contains %lld items. [[%s]]"),
					ArrayIndex, *WriteToString<32>(Name), Array.Num(), StringCast<TCHAR>(Format).Get());
				for (FCbFieldView ArrayIt : Array)
				{
					if (ArrayIndex-- == 0)
					{
						Field = ArrayIt;
						break;
					}
				}
			}

			if (!bMore)
			{
				break;
			}

			Name = Path;
			bMore = String::SplitFirstChar(Path, UTF8CHAR(GFieldPathDelimiter), Name, Path);
			bArray = Name.EndsWith(']') && String::SplitLastChar(Name, UTF8CHAR('['), Name, IndexDigits);
			Field = Field.AsObjectView().FindView(Name);
			checkf(Field, TEXT("Log format requires field '%s' which was not provided. [[%s]]"),
				*WriteToString<32>(Path), StringCast<TCHAR>(Format).Get());
		}
		return Field;
	};

private:
	const FormatCharType* Format;
	const FCbFieldViewIterator Fields;
	FCbFieldViewIterator It{Fields};
	int32 Index{0};
};

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * A table of localized log formats referenced by log templates.
 */
class FLocalizedLogFormatTable
{
public:
	int32 Add(FTextFormat&& Format)
	{
		TUniqueLock Lock(Mutex);
		return Table.Emplace(MoveTemp(Format));
	}

	void RemoveAt(int32 Index)
	{
		TUniqueLock Lock(Mutex);
		Table.RemoveAt(Index);
	}

	FTextFormat Get(int32 Index) const
	{
		TUniqueLock Lock(Mutex);
		return Table[Index];
	}

private:
	TSparseArray<FTextFormat> Table;
	mutable FTransactionallySafeMutex Mutex;
};

static FLocalizedLogFormatTable& GetLocalizedLogFormatTable()
{
	static FLocalizedLogFormatTable Table;
	return Table;
}

} // UE::Logging::Private

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

namespace UE
{

class FLogTemplate
{
	using FLogField = Logging::Private::FLogField;

public:
	template <typename FormatCharType>
	static FLogTemplate* Create(
		const FormatCharType* Format,
		const FLogTemplateOptions& Options,
		const FLogField* Fields,
		int32 FieldCount,
		TFunctionWithContext<void* (int32)> Allocate);

	template <typename FormatCharType>
	static FLogTemplate* CreateLocalized(
		const FText& FormatText,
		const FormatCharType* Format,
		const FLogTemplateOptions& Options,
		const FLogField* Fields,
		int32 FieldCount,
		TFunctionWithContext<void* (int32)> Allocate);

	static void Destroy(FLogTemplate* Template);

	const TCHAR* GetFormat() const { return StaticFormat; }
	const UTF8CHAR* GetUtf8Format() const { return StaticFormatUtf8; }

	uint8* GetOpData() { return (uint8*)(this + 1); }
	const uint8* GetOpData() const { return (const uint8*)(this + 1); }

	template <typename CharType>
	void FormatTo(TStringBuilderBase<CharType>& Out, const FCbFieldViewIterator& Fields) const;

	template <typename FormatCharType, typename OutputCharType>
	static void FormatTo(
		const FormatCharType* Format,
		TStringBuilderBase<OutputCharType>& Out,
		const FCbFieldViewIterator& Fields,
		const uint8* FirstOp);

	FText FormatToText(const FCbFieldViewIterator& Fields) const;

private:
	template <typename CharType>
	void FormatLocalizedTo(TStringBuilderBase<CharType>& Out, const FCbFieldViewIterator& Fields) const;

	template <typename FormatCharType, typename OutputCharType>
	static void FormatLocalizedTo(
		const FormatCharType* const Format,
		TStringBuilderBase<OutputCharType>& Out,
		const FCbFieldViewIterator& Fields,
		const uint8* const FirstOp);

	FText FormatLocalizedToText(const FCbFieldViewIterator& Fields) const;

	template <typename FormatCharType>
	static FText FormatLocalizedToText(
		const FormatCharType* Format,
		const FCbFieldViewIterator& Fields,
		const uint8* FirstOp);

	inline constexpr explicit FLogTemplate(const TCHAR* Format)
		: StaticFormat(Format)
	{
	}

	inline constexpr explicit FLogTemplate(const UTF8CHAR* Format)
		: StaticFormatUtf8(Format)
	{
	}

	FLogTemplate(const FLogTemplate&) = delete;
	FLogTemplate& operator=(const FLogTemplate&) = delete;

	const TCHAR* StaticFormat = nullptr;
	const UTF8CHAR* StaticFormatUtf8 = nullptr;
};

static_assert(std::is_trivially_destructible_v<FLogTemplate>);

template <typename FormatCharType>
FLogTemplate* FLogTemplate::Create(
	const FormatCharType* Format,
	const FLogTemplateOptions& Options,
	const FLogField* Fields,
	const int32 FieldCount,
	TFunctionWithContext<void* (int32)> Allocate)
{
	using namespace Logging::Private;

	const TConstArrayView<FLogField> FieldsView(Fields, FieldCount);
	const bool bFindFields = !!Fields;
	const bool bPositional = !FieldCount || Algo::NoneOf(FieldsView, &FLogField::Name);
	checkf(bPositional || Algo::AllOf(FieldsView, &FLogField::Name),
		TEXT("Log fields must be entirely named or entirely anonymous. [[%s]]"), StringCast<TCHAR>(Format).Get());
	checkf(bPositional || Algo::AllOf(FieldsView,
		[](const FLogField& Field) { return *Field.Name && *Field.Name != '_' && FAsciiSet::HasOnly(Field.Name, GValidLogFieldName); }),
		TEXT("Log field names must match \"[A-Za-z0-9][A-Za-z0-9_]*\" in [[%s]]."), StringCast<TCHAR>(Format).Get());

	TArray<FLogTemplateOp, TInlineAllocator<16>> Ops;

	TAnsiStringBuilder<256> FieldPathData;
	TArray<int32, TInlineAllocator<16>> FieldPathSizes;

	int32 FieldSearchIndex = -1;
	int32 FormatFieldCount = 0;
	int32 SymbolSearchOffset = 0;
	for (const FormatCharType* TextStart = Format;;)
	{
		constexpr FAsciiSet Brackets("{}");
		const FormatCharType* const TextEnd = FAsciiSet::FindFirstOrEnd(TextStart + SymbolSearchOffset, Brackets);
		SymbolSearchOffset = 0;

		// Escaped "{{" or "}}"
		if ((TextEnd[0] == '{' && TextEnd[1] == '{') ||
			(TextEnd[0] == '}' && TextEnd[1] == '}'))
		{
			// Only "{{" or "}}"
			if (TextStart == TextEnd)
			{
				Ops.Add({FLogTemplateOp::OpSkip, 1});
				TextStart = TextEnd + 1;
				SymbolSearchOffset = 1;
			}
			// Text and "{{" or "}}"
			else
			{
				Ops.Add({FLogTemplateOp::OpText, UE_PTRDIFF_TO_INT32(1 + TextEnd - TextStart)});
				Ops.Add({FLogTemplateOp::OpSkip, 1});
				TextStart = TextEnd + 2;
			}
			continue;
		}

		// Text
		if (TextStart != TextEnd)
		{
			Ops.Add({FLogTemplateOp::OpText, UE_PTRDIFF_TO_INT32(TextEnd - TextStart)});
		}

		// End
		if (!*TextEnd)
		{
			Ops.Add({FLogTemplateOp::OpEnd});
			break;
		}

		// Parse and validate the field path.
		const FormatCharType* const FieldStart = TextEnd;
		checkf(*FieldStart == '{', TEXT("Log format has an unexpected '%c' character. Use '%c%c' to escape it. [[%s]]"),
			*FieldStart, *FieldStart, *FieldStart, StringCast<TCHAR>(Format).Get());
		const FormatCharType* const FieldPathEnd = FAsciiSet::Skip(FieldStart + 1, GValidLogFieldPath);
		checkf(*FieldPathEnd, TEXT("Log format has an unterminated field reference. Use '{{' to escape '{' if needed. [[%s]]"),
			StringCast<TCHAR>(Format).Get());
		checkf(*FieldPathEnd == '}', TEXT("Log format has invalid character '%c' in field name. "
			"Use '{{' to escape '{' if needed. Names must match \"[A-Za-z0-9][A-Za-z0-9_]*\". [[%s]]"),
			*FieldPathEnd, StringCast<TCHAR>(Format).Get());
		const FormatCharType* const FieldEnd = FieldPathEnd + 1;
		const int32 FieldLen = UE_PTRDIFF_TO_INT32(FieldEnd - FieldStart);
		checkf(FieldStart[1] != '_', TEXT("Log format uses reserved field name '%s' with leading '_'. "
			"Names must match \"[A-Za-z0-9][A-Za-z0-9_]*\". [[%s]]"),
			*WriteToString<32>(MakeStringView(FieldStart + 1, FieldLen - 2)), StringCast<TCHAR>(Format).Get());

		const int32 FieldPathIndex = FieldPathData.Len();
		FieldPathData.Append(FieldStart + 1, FieldLen - 2);
		const FAnsiStringView FieldPath = FieldPathData.ToView().RightChop(FieldPathIndex);
		FieldPathSizes.Add(FieldPath.Len());

		const bool bHasSubObjectReference = FAsciiSet::HasAny(FieldPath, GValidLogFieldPath & ~GValidLogFieldName);
		checkf(!bHasSubObjectReference || Options.bAllowSubObjectReferences,
			TEXT("Log format has a sub-object reference (%c or []) in a context that does not allow them. [[%s]]"),
			GFieldPathDelimiter, StringCast<TCHAR>(Format).Get());

		if (bFindFields && !bPositional)
		{
			bool bFoundField = false;
			for (int32 SearchCount = FieldCount; SearchCount > 0; --SearchCount)
			{
				FieldSearchIndex = (FieldSearchIndex + 1) % FieldCount;
				if (FieldPath.Equals(Fields[FieldSearchIndex].Name))
				{
					Ops.Add({FLogTemplateOp::OpIndex, FieldSearchIndex});
					bFoundField = true;
					break;
				}
			}
			checkf(bFoundField, TEXT("Log format requires field '%.*hs' which was not provided. [[%s]]"),
				FieldPath.Len(), FieldPath.GetData(), StringCast<TCHAR>(Format).Get());
		}

		Ops.Add({bHasSubObjectReference ? FLogTemplateOp::OpPath : FLogTemplateOp::OpName, FieldLen});
		++FormatFieldCount;

		TextStart = FieldEnd;
	}

	checkf(!bFindFields || !bPositional || FormatFieldCount == FieldCount,
		TEXT("Log format requires %d fields and %d were provided. [[%s]]"),
		FormatFieldCount, FieldCount, StringCast<TCHAR>(Format).Get());

	const uint32 TotalSize = sizeof(FLogTemplate) + Algo::TransformAccumulate(Ops, FLogTemplateOp::SaveSize, 0);
	FLogTemplate* const Template = new(Allocate(TotalSize)) FLogTemplate(Format);
	uint8* Data = Template->GetOpData();
	for (const FLogTemplateOp& Op : Ops)
	{
		FLogTemplateOp::Save(Op, Data);
	}
	return Template;
}

template <typename FormatCharType>
FLogTemplate* FLogTemplate::CreateLocalized(
	const FText& FormatText,
	const FormatCharType* Format,
	const FLogTemplateOptions& Options,
	const FLogField* Fields,
	const int32 FieldCount,
	TFunctionWithContext<void* (int32)> Allocate)
{
	// A localized format string consists of an OpLocalized op followed by a sequence of OpSkip and OpName/OpPath ops
	// that are terminated by an OpEnd op. Only the first occurrence of each name/path is included and everything else
	// in the format string is skipped. Anything following the last name/path is ignored and not even skipped.

	using namespace Logging::Private;

	const bool bFindFields = !!Fields;
	checkf(!bFindFields || !Options.bAllowSubObjectReferences,
		TEXT("Validation of field names is not compatible with sub-object references. [[%s]]"), StringCast<TCHAR>(Format).Get());

	TArray<FLogTemplateOp, TInlineAllocator<16>> Ops;
	Ops.Add({FLogTemplateOp::OpLocalized, GetLocalizedLogFormatTable().Add(FTextFormat(FormatText))});

	// Track unique field names to avoid adding multiple ops for the same name.
	TAnsiStringBuilder<256> FieldPathData;
	TArray<int32, TInlineAllocator<16>> FieldPathSizes;

	// Parse the format string to find unique field names and optionally validate that required fields are present.
	int32 FieldSearchIndex = -1;
	int32 SymbolSearchOffset = 0;
	for (const FormatCharType* TextStart = Format;;)
	{
		constexpr FAsciiSet Symbols("`{}");
		const FormatCharType* const TextEnd = FAsciiSet::FindFirstOrEnd(TextStart + SymbolSearchOffset, Symbols);
		SymbolSearchOffset = 0;

		// Escaped "``" or "`{" or "`}"
		if (TextEnd[0] == '`' && (TextEnd[1] == '`' || TextEnd[1] == '{' || TextEnd[1] == '}'))
		{
			// Continue the search after the escaped symbol.
			SymbolSearchOffset = UE_PTRDIFF_TO_INT32(2 + TextEnd - TextStart);
			continue;
		}

		// End. Implicitly skips any text after the last field path.
		if (!*TextEnd)
		{
			Ops.Add({FLogTemplateOp::OpEnd});
			break;
		}

		// Parse and validate the field path.
		const FormatCharType* const FieldStart = TextEnd;
		checkf(*FieldStart == '{', TEXT("Log format has an unexpected '%c' character. Use '`%c' to escape it. [[%s]]"),
			*FieldStart, *FieldStart, StringCast<TCHAR>(Format).Get());
		const FormatCharType* const FieldPathEnd = FAsciiSet::Skip(FieldStart + 1, GValidLogFieldPath);
		checkf(*FieldPathEnd, TEXT("Log format has an unterminated field reference. Use '`{' to escape '{' if needed. [[%s]]"),
			StringCast<TCHAR>(Format).Get());
		checkf(*FieldPathEnd == '}', TEXT("Log format has invalid character '%c' in field name. "
			"Use '`{' to escape '{' if needed. Names must match \"[A-Za-z0-9][A-Za-z0-9_]*\". [[%s]]"),
			*FieldPathEnd, StringCast<TCHAR>(Format).Get());
		const FormatCharType* const FieldEnd = FieldPathEnd + 1;
		const int32 FieldLen = UE_PTRDIFF_TO_INT32(FieldEnd - FieldStart);
		checkf(FieldStart[1] != '_', TEXT("Log format uses reserved field name '%s' with leading '_'. "
			"Names must match \"[A-Za-z0-9][A-Za-z0-9_]*\". [[%s]]"),
			*WriteToString<32>(MakeStringView(FieldStart + 1, FieldLen - 2)), StringCast<TCHAR>(Format).Get());

		const int32 FieldPathIndex = FieldPathData.Len();
		FieldPathData.Append(FieldStart + 1, FieldLen - 2);
		const FAnsiStringView FieldPath = FieldPathData.ToView().RightChop(FieldPathIndex);

		const bool bHasSubObjectReference = FAsciiSet::HasAny(FieldPath, GValidLogFieldPath & ~GValidLogFieldName);
		checkf(!bHasSubObjectReference || Options.bAllowSubObjectReferences,
			TEXT("Log format has a sub-object reference (%c or []) in a context that does not allow them. [[%s]]"),
			GFieldPathDelimiter, StringCast<TCHAR>(Format).Get());

		// Check if the field path has been seen and skip it if it has.
		const bool bExistingField = !!Algo::FindByPredicate(FieldPathSizes, [FieldPath, Data = FieldPathData.GetData()](int32 Size) mutable
		{
			ON_SCOPE_EXIT { Data += Size; };
			return FieldPath.Equals(MakeStringView(Data, Size));
		});
		if (bExistingField)
		{
			// Continue the search after the repeated field path.
			SymbolSearchOffset = UE_PTRDIFF_TO_INT32(FieldEnd - TextStart);
			FieldPathData.RemoveSuffix(FieldPath.Len());
			continue;
		}
		FieldPathSizes.Add(FieldPath.Len());

		// Skip the text along with any escaped symbols and repeated field paths.
		if (TextStart != TextEnd)
		{
			Ops.Add({FLogTemplateOp::OpSkip, UE_PTRDIFF_TO_INT32(TextEnd - TextStart)});
		}

		if (bFindFields)
		{
			bool bFoundField = false;
			for (int32 SearchCount = FieldCount; SearchCount > 0; --SearchCount)
			{
				FieldSearchIndex = (FieldSearchIndex + 1) % FieldCount;
				if (FieldPath.Equals(Fields[FieldSearchIndex].Name))
				{
					Ops.Add({FLogTemplateOp::OpIndex, FieldSearchIndex});
					bFoundField = true;
					break;
				}
			}
			checkf(bFoundField, TEXT("Log format requires field '%.*hs' which was not provided. [[%s]]"),
				FieldPath.Len(), FieldPath.GetData(), StringCast<TCHAR>(Format).Get());
		}

		Ops.Add({bHasSubObjectReference ? FLogTemplateOp::OpPath : FLogTemplateOp::OpName, FieldLen});

		TextStart = FieldEnd;
	}

	const uint32 TotalSize = sizeof(FLogTemplate) + Algo::TransformAccumulate(Ops, FLogTemplateOp::SaveSize, 0);
	FLogTemplate* const Template = new(Allocate(TotalSize)) FLogTemplate(Format);
	uint8* Data = Template->GetOpData();
	for (const FLogTemplateOp& Op : Ops)
	{
		FLogTemplateOp::Save(Op, Data);
	}
	return Template;
}

void FLogTemplate::Destroy(FLogTemplate* Template)
{
	using namespace Logging::Private;

	const uint8* NextOp = Template->GetOpData();
	if (FLogTemplateOp Op = FLogTemplateOp::Load(NextOp); Op.Code == FLogTemplateOp::OpLocalized)
	{
		GetLocalizedLogFormatTable().RemoveAt(Op.Value);
	}
}

template <typename CharType>
void FLogTemplate::FormatTo(TStringBuilderBase<CharType>& Out, const FCbFieldViewIterator& Fields) const
{
	if (StaticFormatUtf8)
	{
		FormatTo(StaticFormatUtf8, Out, Fields, GetOpData());
	}
	else
	{
		FormatTo(StaticFormat, Out, Fields, GetOpData());
	}
}

template <typename FormatCharType, typename OutputCharType>
void FLogTemplate::FormatTo(
	const FormatCharType* const Format,
	TStringBuilderBase<OutputCharType>& Out,
	const FCbFieldViewIterator& Fields,
	const uint8* const FirstOp)
{
	using namespace Logging::Private;

	int32 FieldIndexHint = -1;
	const uint8* NextOp = FirstOp;
	const FormatCharType* NextFormat = Format;
	TFieldFinder FieldFinder(Format, Fields);
	for (;;)
	{
		const FLogTemplateOp Op = FLogTemplateOp::Load(NextOp);
		switch (Op.Code)
		{
		case FLogTemplateOp::OpLocalized:
			return FormatLocalizedTo(Format, Out, Fields, FirstOp);
		case FLogTemplateOp::OpEnd:
			return;
		case FLogTemplateOp::OpText:
			Out.Append(NextFormat, Op.Value);
			break;
		case FLogTemplateOp::OpIndex:
			FieldIndexHint = Op.Value;
			break;
		case FLogTemplateOp::OpName:
			{
				const auto Name = StringCast<UTF8CHAR, 32>(NextFormat + 1, Op.Value - 2);
				LogFieldValue(Out, FieldFinder.Find(Name, FieldIndexHint));
				FieldIndexHint = -1;
			}
			break;
		case FLogTemplateOp::OpPath:
			{
				const auto Path = StringCast<UTF8CHAR, 32>(NextFormat + 1, Op.Value - 2);
				LogFieldValue(Out, FieldFinder.FindByPath(Path, FieldIndexHint));
				FieldIndexHint = -1;
			}
			break;
		}
		NextFormat += Op.GetSkipSize();
	}
}

FText FLogTemplate::FormatToText(const FCbFieldViewIterator& Fields) const
{
	using namespace Logging::Private;

	const uint8* NextOp = GetOpData();
	if (FLogTemplateOp::Load(NextOp).Code == FLogTemplateOp::OpLocalized)
	{
		return FormatLocalizedToText(Fields);
	}
	else
	{
		TStringBuilder<512> Builder;
		FormatTo(Builder, Fields);
		return FText::FromStringView(Builder);
	}
}

template <typename CharType>
FORCENOINLINE void FLogTemplate::FormatLocalizedTo(TStringBuilderBase<CharType>& Out, const FCbFieldViewIterator& Fields) const
{
	Out.Append(FormatLocalizedToText(Fields).ToString());
}

template <typename FormatCharType, typename OutputCharType>
FORCENOINLINE void FLogTemplate::FormatLocalizedTo(
	const FormatCharType* const Format,
	TStringBuilderBase<OutputCharType>& Out,
	const FCbFieldViewIterator& Fields,
	const uint8* const FirstOp)
{
	Out.Append(FormatLocalizedToText(Format, Fields, FirstOp).ToString());
}

FText FLogTemplate::FormatLocalizedToText(const FCbFieldViewIterator& Fields) const
{
	if (StaticFormatUtf8)
	{
		return FormatLocalizedToText(StaticFormatUtf8, Fields, GetOpData());
	}
	else
	{
		return FormatLocalizedToText(StaticFormat, Fields, GetOpData());
	}
}

template <typename FormatCharType>
FText FLogTemplate::FormatLocalizedToText(
	const FormatCharType* const Format,
	const FCbFieldViewIterator& Fields,
	const uint8* const FirstOp)
{
	using namespace Logging::Private;

	TOptional<FTextFormat> TextFormat;
	FFormatNamedArguments TextFormatArguments;

	int32 FieldIndexHint = -1;
	const uint8* NextOp = FirstOp;
	const FormatCharType* NextFormat = Format;
	TFieldFinder FieldFinder(Format, Fields);
	while (NextOp)
	{
		const FLogTemplateOp Op = FLogTemplateOp::Load(NextOp);
		switch (Op.Code)
		{
		case FLogTemplateOp::OpLocalized:
			TextFormat = GetLocalizedLogFormatTable().Get(Op.Value);
			break;
		case FLogTemplateOp::OpEnd:
			NextOp = nullptr;
			break;
		case FLogTemplateOp::OpIndex:
			FieldIndexHint = Op.Value;
			break;
		case FLogTemplateOp::OpName:
			{
				const auto Name = StringCast<UTF8CHAR, 32>(NextFormat + 1, Op.Value - 2);
				AddFieldValue(TextFormatArguments, Name, FieldFinder.Find(Name, FieldIndexHint));
				FieldIndexHint = -1;
			}
			break;
		case FLogTemplateOp::OpPath:
			{
				const auto Path = StringCast<UTF8CHAR, 32>(NextFormat + 1, Op.Value - 2);
				AddFieldValue(TextFormatArguments, Path, FieldFinder.FindByPath(Path, FieldIndexHint));
				FieldIndexHint = -1;
			}
			break;
		}
		NextFormat += Op.GetSkipSize();
	}

	checkf(TextFormat, TEXT("Missing text format when formatting localized template. [[%s]]"), StringCast<TCHAR>(Format).Get());
	return FText::Format(*TextFormat, TextFormatArguments);
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

void Logging::Private::CreateLogTemplate(const TCHAR* Format, const FLogTemplateOptions& Options, const FLogField* Fields, const int32 FieldCount, TFunctionWithContext<void* (int32)> Allocate)
{
	FLogTemplate::Create(Format, Options, Fields, FieldCount, Allocate);
}

void Logging::Private::CreateLogTemplate(const UTF8CHAR* Format, const FLogTemplateOptions& Options, const FLogField* Fields, const int32 FieldCount, TFunctionWithContext<void* (int32)> Allocate)
{
	FLogTemplate::Create(Format, Options, Fields, FieldCount, Allocate);
}

void Logging::Private::CreateLocalizedLogTemplate(const FText& Format, const FLogTemplateOptions& Options, const FLogField* Fields, const int32 FieldCount, TFunctionWithContext<void* (int32)> Allocate)
{
	FLogTemplate::CreateLocalized(Format, *Format.ToString(), Options, Fields, FieldCount, Allocate);
}

void Logging::Private::CreateLocalizedLogTemplate(const TCHAR* TextNamespace, const TCHAR* TextKey, const TCHAR* Format, const FLogTemplateOptions& Options, const FLogField* Fields, const int32 FieldCount, TFunctionWithContext<void* (int32)> Allocate)
{
	const FText FormatText = FText::AsLocalizable_Advanced(TextNamespace, TextKey, Format);
	FLogTemplate::CreateLocalized(FormatText, Format, Options, Fields, FieldCount, Allocate);
}

void Logging::Private::CreateLocalizedLogTemplate(const TCHAR* TextNamespace, const TCHAR* TextKey, const UTF8CHAR* Format, const FLogTemplateOptions& Options, const FLogField* Fields, const int32 FieldCount, TFunctionWithContext<void* (int32)> Allocate)
{
	const FText FormatText = FText::AsLocalizable_Advanced(TextNamespace, TextKey, Format);
	FLogTemplate::CreateLocalized(FormatText, Format, Options, Fields, FieldCount, Allocate);
}

void Logging::Private::DestroyLogTemplate(FLogTemplate* Template)
{
	if (Template)
	{
		FLogTemplate::Destroy(Template);
	}
}

void FormatLogTo(FUtf8StringBuilderBase& Out, const FLogTemplate* Template, const FCbFieldViewIterator& Fields)
{
	Template->FormatTo(Out, Fields);
}

void FormatLogTo(FWideStringBuilderBase& Out, const FLogTemplate* Template, const FCbFieldViewIterator& Fields)
{
	Template->FormatTo(Out, Fields);
}

FText FormatLogToText(const FLogTemplate* Template, const FCbFieldViewIterator& Fields)
{
	return Template->FormatToText(Fields);
}

void SerializeLogFormat(FCbWriter& Writer, const FText& Format)
{
	const TOptional<FString> Namespace = FTextInspector::GetNamespace(Format);
	const TOptional<FString> Key = FTextInspector::GetKey(Format);
	const FString* Source = FTextInspector::GetSourceString(Format);
	checkf(Namespace && Key && Source, TEXT("Serializing a localized format string requires a namespace, key, and source string. [[%s]]"), *Format.ToString());
	Writer.AddString(ANSITEXTVIEW("$locformat"), *Source);
	Writer.AddString(ANSITEXTVIEW("$locns"), *Namespace);
	Writer.AddString(ANSITEXTVIEW("$lockey"), *Key);
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FLogTime FLogTime::Now()
{
	FLogTime Time;
	Time.UtcTicks = FDateTime::UtcNow().GetTicks();
	return Time;
}

FLogTime FLogTime::FromUtcTime(const FDateTime& UtcTime)
{
	FLogTime Time;
	Time.UtcTicks = UtcTime.GetTicks();
	return Time;
}

FDateTime FLogTime::GetUtcTime() const
{
	return FDateTime(UtcTicks);
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

template <typename CharType>
FORCENOINLINE static void FormatDynamicRecordMessageTo(TStringBuilderBase<CharType>& Out, const FLogRecord& Record)
{
	const UTF8CHAR* Utf8Format = Record.GetUtf8Format();
	PRAGMA_DISABLE_DEPRECATION_WARNINGS;
	const TCHAR* Format = Record.GetFormat();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS;
	if (UNLIKELY(!Utf8Format && !Format))
	{
		return;
	}

	const TCHAR* TextNamespace = Record.GetTextNamespace();
	const TCHAR* TextKey = Record.GetTextKey();
	checkf(!TextNamespace == !TextKey,
		TEXT("Log record must have both or neither of the text namespace and text key. [[%s]]"),
		Utf8Format ? StringCast<TCHAR>(Utf8Format).Get() : Format);

	TOptional<FInlineLogTemplate> LocalTemplate;
	if (LIKELY(Utf8Format))
	{
		TextKey ? LocalTemplate.Emplace(TextNamespace, TextKey, Utf8Format) : LocalTemplate.Emplace(Utf8Format);
	}
	else
	{
		TextKey ? LocalTemplate.Emplace(TextNamespace, TextKey, Format) : LocalTemplate.Emplace(Format);
	}
	LocalTemplate->FormatTo(Out, Record.GetFields().CreateViewIterator());
}

template <typename CharType>
static void FormatRecordMessageTo(TStringBuilderBase<CharType>& Out, const FLogRecord& Record)
{
	using namespace Logging::Private;

	if (GPrependLogContextToLogMessage)
	{
		const FCbObject& Fields = Record.GetFields();
		for (FCbFieldView NameField : Fields[GLogContextsFieldName])
		{
			const FUtf8StringView NameView = NameField.AsString();
			if (FCbFieldView ContextField = Fields[NameView])
			{
				Out.Append(NameView);
				if (!ContextField.IsNull())
				{
					Out.AppendChar('(');
					CompactBinaryToCompactJson(ContextField.RemoveName(), Out);
					Out.AppendChar(')');
				}
				Out.AppendChar(':');
				Out.AppendChar(' ');
			}
		}
	}

	const FLogTemplate* Template = Record.GetTemplate();
	if (LIKELY(Template))
	{
		return Template->FormatTo(Out, Record.GetFields().CreateViewIterator());
	}
	FormatDynamicRecordMessageTo(Out, Record);
}

void FLogRecord::FormatMessageTo(FUtf8StringBuilderBase& Out) const
{
	FormatRecordMessageTo(Out, *this);
}

void FLogRecord::FormatMessageTo(FWideStringBuilderBase& Out) const
{
	FormatRecordMessageTo(Out, *this);
}

void FLogRecord::ConvertToCommonLog(FUtf8StringBuilderBase& OutFormat, FCbWriter& OutFields) const
{
	using namespace Logging::Private;

	for (FCbField& Field : Fields)
	{
		OutFields.SetName(Field.GetName());
		if (FCbArray Array = Field.AsArray(); !Field.HasError())
		{
			OutFields.BeginObject();
			OutFields.AddArray(ANSITEXTVIEW("$value"), Array);
			TUtf8StringBuilder<256> Text;
			LogFieldValue(Text, Field);
			OutFields.AddString(ANSITEXTVIEW("$text"), Text);
			OutFields.EndObject();
		}
		else if (FCbObject Object = Field.AsObject(); !Field.HasError() && !Object.FindView(ANSITEXTVIEW("$text")))
		{
			OutFields.BeginObject();
			for (FCbField& Child : Object)
			{
				OutFields.AddField(Child.GetName(), Child);
			}
			TUtf8StringBuilder<256> Text;
			LogFieldValue(Text, Field);
			OutFields.AddString(ANSITEXTVIEW("$text"), Text);
			OutFields.EndObject();
		}
		else
		{
			OutFields.AddField(Field);
		}
	}

	// TODO: Process localized format strings to remove argument modifiers and convert escaped braces.
	if (LIKELY(Utf8Format))
	{
		OutFormat.Append(Utf8Format);
	}
	else if (Format)
	{
		OutFormat.Append(Format);
	}
}

} // UE

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

namespace UE::Logging::Private
{

#if NO_LOGGING
FLogCategory<ELogVerbosity::Fatal, ELogVerbosity::Fatal> LogFatal(TEXT("Fatal"));
#endif

thread_local FLogContext* LogContextHead = nullptr;
thread_local FLogContext* LogContextTail = nullptr;

FLogContext::FLogContext(const ANSICHAR* InName)
	: FLogContext(InName, nullptr, nullptr)
{
}

FLogContext::FLogContext(const ANSICHAR* InName, const void* InValue, FLogField::FWriteFn* InWriteValue)
: Name(InName)
, Prev(LogContextTail)
{
	(Prev ? Prev->Next : LogContextHead) = this;
	LogContextTail = this;

	TCbWriter<256> Writer;
	if (InWriteValue)
	{
		InWriteValue(Writer, InValue);
	}
	else
	{
		Writer.AddNull();
	}
	Value.SetNumUninitialized((int32)Writer.GetSaveSize());
	Writer.Save(MakeMemoryView(Value));

	for (FLogContext* Node = Prev; Node; Node = Node->Prev)
	{
		if (FCStringAnsi::Strcmp(Name, Node->Name) == 0)
		{
			Node->bDisabledByNewerContext = true;
			bDisabledOlderContext = true;
		}
	}
}

FLogContext::~FLogContext()
{
	(Prev ? Prev->Next : LogContextHead) = Next;
	(Next ? Next->Prev : LogContextTail) = Prev;

	if (bDisabledOlderContext)
	{
		for (FLogContext* Node = Prev; Node; Node = Node->Prev)
		{
			if (FCStringAnsi::Strcmp(Name, Node->Name) == 0)
			{
				Node->bDisabledByNewerContext = false;
			}
		}
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

class FLogTemplateFieldIterator
{
public:
	inline explicit FLogTemplateFieldIterator(const FLogTemplate& Template)
		: NextOp(Template.GetOpData())
		, NextFormat(Template.GetUtf8Format())
	{
		++*this;
	}

	FLogTemplateFieldIterator& operator++();
	inline explicit operator bool() const { return !!NextOp; }
	inline const FUtf8StringView& GetName() const { return Name; }

private:
	FUtf8StringView Name;
	const uint8* NextOp = nullptr;
	const UTF8CHAR* NextFormat = nullptr;
};

FLogTemplateFieldIterator& FLogTemplateFieldIterator::operator++()
{
	using namespace Logging::Private;

	while (NextOp)
	{
		FLogTemplateOp Op = FLogTemplateOp::Load(NextOp);
		if (Op.Code == FLogTemplateOp::OpName)
		{
			Name = FUtf8StringView(NextFormat + 1, Op.Value - 2);
			NextFormat += Op.GetSkipSize();
			return *this;
		}
		if (Op.Code == FLogTemplateOp::OpEnd)
		{
			break;
		}
		NextFormat += Op.GetSkipSize();
	}

	NextOp = nullptr;
	Name.Reset();
	return *this;
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

#if LOGTRACE_ENABLED
template <typename StaticLogRecordType>
FORCENOINLINE static void EnsureLogMessageSpec(const FLogCategoryBase& Category, const StaticLogRecordType& Log, ELogVerbosity::Type Verbosity)
{
	if (!Log.DynamicData.bInitializedTrace.load(std::memory_order_acquire))
	{
		FLogTrace::OutputLogMessageSpec(&Log, &Category, Verbosity, Log.File, Log.Line, TEXT("%s"));
		Log.DynamicData.bInitializedTrace.store(true, std::memory_order_release);
	}
}

// Tracing the log happens in its own function because that allows stack space for the message to
// be returned before calling into the output devices.
FORCENOINLINE static void LogToTrace(const void* LogPoint, const FLogRecord& Record)
{
	TStringBuilder<1024> Message;
	Record.FormatMessageTo(Message);
	FLogTrace::OutputLogMessage(LogPoint, *Message);
}

// Tracing the log happens in its own function because that allows stack space for the message to
// be returned before calling into the output devices.
FORCENOINLINE static void BasicLogToTrace(const void* LogPoint, const TCHAR* Format, va_list Args)
{
	TStringBuilder<1024> Message;
	Message.AppendV(Format, Args);
	FLogTrace::OutputLogMessage(LogPoint, *Message);
}
#endif // LOGTRACE_ENABLED

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

// Serializing log fields to compact binary happens in its own function because that allows stack
// space for the writer to be returned before calling into the output devices.
FORCENOINLINE static FCbObject SerializeLogFields(
	const FLogTemplate& Template,
	const FLogField* Fields,
	const int32 FieldCount)
{
	const bool bHasContext = !!LogContextHead;
	if (FieldCount == 0 && !bHasContext)
	{
		return FCbObject();
	}

	TCbWriter<1024> Writer;
	Writer.BeginObject();

	TArray<FUtf8StringView, TInlineAllocator<16>> FieldNames;

	// Anonymous. Extract names from Template.
	if (!Fields->Name)
	{
		FLogTemplateFieldIterator It(Template);
		for (const FLogField* FieldsEnd = Fields + FieldCount; Fields != FieldsEnd; ++Fields, ++It)
		{
			check(It);
			Fields->WriteValue(Writer.SetName(It.GetName()), Fields->Value);
			if (bHasContext)
			{
				FieldNames.Emplace(It.GetName());
			}
		}
		check(!It);
	}
	// Named
	else
	{
		for (const FLogField* FieldsEnd = Fields + FieldCount; Fields != FieldsEnd; ++Fields)
		{
			Fields->WriteValue(Writer.SetName(Fields->Name), Fields->Value);
			if (bHasContext)
			{
				FieldNames.Emplace(Fields->Name);
			}
		}
	}

	if (bHasContext)
	{
		TBitArray<> ActiveContext;
		int32 ContextIndex = 0;

		// Traverse contexts backward and activate any which have a name that has not been seen yet.
		for (FLogContext* Node = LogContextTail; Node; Node = Node->Prev)
		{
			if (!Node->bDisabledByNewerContext)
			{
				const FAnsiStringView NodeName = Node->Name;
				ActiveContext.Add(!FieldNames.FindByPredicate([NodeName](FUtf8StringView Name) { return NodeName.Equals(Name); }));
				++ContextIndex;
			}
		}

		// Traverse contexts forward and copy any which were activated above.
		for (FLogContext* Node = LogContextHead; Node; Node = Node->Next)
		{
			if (!Node->bDisabledByNewerContext && ActiveContext[--ContextIndex])
			{
				Writer.AddField(Node->Name, FCbFieldView(Node->Value.GetData()));
			}
		}

		// Traverse contexts forward and build an array of names in $Contexts.
		Writer.BeginArray(GLogContextsFieldName);
		for (FLogContext* Node = LogContextHead; Node; Node = Node->Next)
		{
			if (!Node->bDisabledByNewerContext)
			{
				Writer.AddString(Node->Name);
			}
		}
		Writer.EndArray();
	}

	Writer.EndObject();
	return Writer.Save().AsObject();
}

inline static FInlineLogTemplate CreateLogTemplate(const FStaticLogRecord& Log, const FLogField* Fields, const int32 FieldCount)
{
	return FInlineLogTemplate(Log.Format, {}, Fields, FieldCount);
}

inline static FInlineLogTemplate CreateLogTemplate(const FStaticLocalizedLogRecord& Log, const FLogField* Fields, const int32 FieldCount)
{
	return FInlineLogTemplate(Log.TextNamespace, Log.TextKey, Log.Format, {}, Fields, FieldCount);
}

template <typename StaticLogRecordType>
inline static FLogRecord CreateLogRecord(const FLogCategoryBase& Category, const StaticLogRecordType& Log, const FLogField* Fields, const int32 FieldCount)
{
#if LOGTRACE_ENABLED
	if (UE_TRACE_CHANNELEXPR_IS_ENABLED(LogChannel))
	{
		EnsureLogMessageSpec(Category, Log, Log.Verbosity);
	}
#endif

	const FInlineLogTemplate Template = CreateLogTemplate(Log, Fields, FieldCount);

	FLogRecord Record;
	Record.SetUtf8Format(Log.Format);
	Record.SetFields(SerializeLogFields(*Template.Get(), Fields, FieldCount));
	Record.SetFile(Log.File);
	Record.SetLine(Log.Line);
	Record.SetCategory(Category.GetCategoryName());
	Record.SetVerbosity(Log.Verbosity);
	Record.SetTime(FLogTime::Now());
	return Record;
}

inline static void DispatchLogRecord(const FLogRecord& Record)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FMsgLogf);

	FOutputDevice* OutputDevice = nullptr;
	switch (Record.GetVerbosity())
	{
	case ELogVerbosity::Error:
	case ELogVerbosity::Warning:
	case ELogVerbosity::Display:
	case ELogVerbosity::SetColor:
		OutputDevice = GWarn;
		break;
	default:
		break;
	}
	(OutputDevice ? OutputDevice : GLog)->SerializeRecord(Record);

#if CSV_PROFILER_STATS
	// Only update the CSV stat if we're not crashing, otherwise things can get messy
	if (LIKELY(!FPlatformMallocCrash::IsActive()))
	{
		CSV_CUSTOM_STAT(FMsgLogf, FMsgLogfCount, 1, ECsvCustomStatOp::Accumulate);
	}
#endif
}

#if !NO_LOGGING

template <typename StaticLogRecordType>
inline static void DispatchStaticLogRecord(const StaticLogRecordType& Log, const FLogRecord& Record)
{
#if LOGTRACE_ENABLED
	if (UE_TRACE_CHANNELEXPR_IS_ENABLED(LogChannel))
	{
		LogToTrace(&Log, Record);
	}
#endif

	DispatchLogRecord(Record);
}

UE_AUTORTFM_ALWAYS_OPEN
void LogWithFieldArray(const FLogCategoryBase& Category, const FStaticLogRecord& Log, const FLogField* Fields, const int32 FieldCount)
{
	DispatchStaticLogRecord(Log, CreateLogRecord(Category, Log, Fields, FieldCount));
}

void LogWithNoFields(const FLogCategoryBase& Category, const FStaticLogRecord& Log)
{
	// A non-null field pointer enables field validation in FLogTemplate::Create.
	static constexpr FLogField EmptyField{};
	LogWithFieldArray(Category, Log, &EmptyField, 0);
}

UE_AUTORTFM_ALWAYS_OPEN
void LogWithFieldArray(const FLogCategoryBase& Category, const FStaticLocalizedLogRecord& Log, const FLogField* Fields, const int32 FieldCount)
{
	FLogRecord Record = CreateLogRecord(Category, Log, Fields, FieldCount);
	Record.SetTextNamespace(Log.TextNamespace);
	Record.SetTextKey(Log.TextKey);
	DispatchStaticLogRecord(Log, Record);
}

void LogWithNoFields(const FLogCategoryBase& Category, const FStaticLocalizedLogRecord& Log)
{
	// A non-null field pointer enables field validation in FLogTemplate::Create.
	static constexpr FLogField EmptyField{};
	LogWithFieldArray(Category, Log, &EmptyField, 0);
}

#endif // !NO_LOGGING

UE_AUTORTFM_ALWAYS_OPEN
[[noreturn]] void FatalLogWithFieldArray(const FLogCategoryBase& Category, const FStaticLogRecord& Log, const FLogField* Fields, const int32 FieldCount)
{
	FLogRecord Record = CreateLogRecord(Category, Log, Fields, FieldCount);
	TStringBuilder<512> Message;
	Record.FormatMessageTo(Message);

	StaticFailDebug(TEXT("Fatal error:"), Log.File, Log.Line, PLATFORM_RETURN_ADDRESS(), *Message);

	UE_DEBUG_BREAK_AND_PROMPT_FOR_REMOTE();
	FDebug::ProcessFatalError(PLATFORM_RETURN_ADDRESS());

	for (;;);
}

void FatalLogWithNoFields(const FLogCategoryBase& Category, const FStaticLogRecord& Log)
{
	// A non-null field pointer enables field validation in FLogTemplate::Create.
	static constexpr FLogField EmptyField{};
	FatalLogWithFieldArray(Category, Log, &EmptyField, 0);
}

UE_AUTORTFM_ALWAYS_OPEN
[[noreturn]] void FatalLogWithFieldArray(const FLogCategoryBase& Category, const FStaticLocalizedLogRecord& Log, const FLogField* Fields, const int32 FieldCount)
{
	FLogRecord Record = CreateLogRecord(Category, Log, Fields, FieldCount);
	TStringBuilder<512> Message;
	Record.FormatMessageTo(Message);

	StaticFailDebug(TEXT("Fatal error:"), Log.File, Log.Line, PLATFORM_RETURN_ADDRESS(), *Message);

	UE_DEBUG_BREAK_AND_PROMPT_FOR_REMOTE();
	FDebug::ProcessFatalError(PLATFORM_RETURN_ADDRESS());

	for (;;);
}

void FatalLogWithNoFields(const FLogCategoryBase& Category, const FStaticLocalizedLogRecord& Log)
{
	// A non-null field pointer enables field validation in FLogTemplate::Create.
	static constexpr FLogField EmptyField{};
	FatalLogWithFieldArray(Category, Log, &EmptyField, 0);
}

} // UE::Logging::Private

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

namespace UE::Logging::Private
{

static constexpr const UTF8CHAR* const GStaticBasicLogFormat = UTF8TEXT("{Message}");

static FLogTemplate* GetStaticBasicLogTemplate()
{
	static FInlineLogTemplate Template(GStaticBasicLogFormat);
	return Template.Get();
}

static void AddContextsToBasicLog(TCbWriter<512>& Writer, FAnsiStringView Ignore1, FAnsiStringView Ignore2 = {})
{
	if (LogContextHead)
	{
		// Traverse contexts forward and copy any that Message did not override.
		for (FLogContext* Node = LogContextHead; Node; Node = Node->Next)
		{
			const FAnsiStringView Name = Node->Name;
			if (!Node->bDisabledByNewerContext && !Name.Equals(Ignore1) && !Name.Equals(Ignore2))
			{
				Writer.AddField(Name, FCbFieldView(Node->Value.GetData()));
			}
		}

		// Traverse contexts forward and build an array of names in $Contexts.
		Writer.BeginArray(GLogContextsFieldName);
		for (FLogContext* Node = LogContextHead; Node; Node = Node->Next)
		{
			if (!Node->bDisabledByNewerContext)
			{
				Writer.AddString(Node->Name);
			}
		}
		Writer.EndArray();
	}
}

// Serializing the log to compact binary happens in its own function because that allows stack
// space for the writer to be returned before calling into the output devices.
template <UE::CCharType FormatCharType>
FORCENOINLINE static FCbObject SerializeBasicLogMessage(TStaticBasicLogRecordParam<FormatCharType> Log, va_list Args)
{
	TStringBuilderWithBuffer<FormatCharType, 512> Message;
	Message.AppendV(Log.Format, Args);

	TCbWriter<512> Writer;
	Writer.BeginObject();

	const FAnsiStringView MessageName = ANSITEXTVIEW("Message");
	Writer.AddString(MessageName, Message);

	AddContextsToBasicLog(Writer, MessageName);

	Writer.EndObject();
	return Writer.Save().AsObject();
}

UE_AUTORTFM_ALWAYS_OPEN
static void BasicLogV(TStaticBasicLogRecordParam<TCHAR> Log, const FLogCategoryBase& Category, ELogVerbosity::Type Verbosity, va_list Args)
{
#if !NO_LOGGING
#if LOGTRACE_ENABLED
	if (UE_TRACE_CHANNELEXPR_IS_ENABLED(LogChannel))
	{
		EnsureLogMessageSpec(Category, Log, Verbosity);
	}
#endif

	if (GConvertBasicLogToLogRecord)
	{
		FCbObject Fields = SerializeBasicLogMessage(Log, Args);

		FLogRecord Record;
		Record.SetUtf8Format(GStaticBasicLogFormat);
		Record.SetTemplate(GetStaticBasicLogTemplate());
		Record.SetFields(MoveTemp(Fields));
		Record.SetFile(Log.File);
		Record.SetLine(Log.Line);
		Record.SetCategory(Category.GetCategoryName());
		Record.SetVerbosity(Verbosity);
		Record.SetTime(FLogTime::Now());

		DispatchStaticLogRecord(Log, Record);
	}
	else
	{
	#if LOGTRACE_ENABLED
		if (UE_TRACE_CHANNELEXPR_IS_ENABLED(LogChannel))
		{
			va_list Args2;
			va_copy(Args2, Args);
			BasicLogToTrace(&Log, Log.Format, Args2);
			va_end(Args2);
		}
	#endif
		FMsg::LogV(Log.File, Log.Line, Category.GetCategoryName(), Verbosity, Log.Format, Args);
	}
#endif
}

template <UE::CCharType FormatCharType>
	requires TIsCharEncodingCompatibleWith_V<FormatCharType, UTF8CHAR>
UE_AUTORTFM_ALWAYS_OPEN
static void BasicLogV(TStaticBasicLogRecordParam<FormatCharType> Log, const FLogCategoryBase& Category, ELogVerbosity::Type Verbosity, va_list Args)
{
#if !NO_LOGGING
#if LOGTRACE_ENABLED
	if (UE_TRACE_CHANNELEXPR_IS_ENABLED(LogChannel))
	{
		EnsureLogMessageSpec(Category, Log, Verbosity);
	}
#endif

	FCbObject Fields = SerializeBasicLogMessage(Log, Args);

	FLogRecord Record;
	Record.SetUtf8Format(GStaticBasicLogFormat);
	Record.SetTemplate(GetStaticBasicLogTemplate());
	Record.SetFields(MoveTemp(Fields));
	Record.SetFile(Log.File);
	Record.SetLine(Log.Line);
	Record.SetCategory(Category.GetCategoryName());
	Record.SetVerbosity(Verbosity);
	Record.SetTime(FLogTime::Now());

	DispatchStaticLogRecord(Log, Record);
#endif
}

template <ELogVerbosity::Type Verbosity, UE::CCharType FormatCharType>
FORCENOINLINE void BasicLog(TStaticBasicLogRecordParam<FormatCharType> Log, const FLogCategoryBase* Category, ...)
{
#if !NO_LOGGING
	va_list Args;
	va_start(Args, Category);
	BasicLogV(Log, *Category, Verbosity, Args);
	va_end(Args);
#endif
}

template CORE_API void BasicLog<ELogVerbosity::Error>(TStaticBasicLogRecordParam<TCHAR> Log, const FLogCategoryBase* Category, ...);
template CORE_API void BasicLog<ELogVerbosity::Warning>(TStaticBasicLogRecordParam<TCHAR> Log, const FLogCategoryBase* Category, ...);
template CORE_API void BasicLog<ELogVerbosity::Display>(TStaticBasicLogRecordParam<TCHAR> Log, const FLogCategoryBase* Category, ...);
template CORE_API void BasicLog<ELogVerbosity::Log>(TStaticBasicLogRecordParam<TCHAR> Log, const FLogCategoryBase* Category, ...);
template CORE_API void BasicLog<ELogVerbosity::Verbose>(TStaticBasicLogRecordParam<TCHAR> Log, const FLogCategoryBase* Category, ...);
template CORE_API void BasicLog<ELogVerbosity::VeryVerbose>(TStaticBasicLogRecordParam<TCHAR> Log, const FLogCategoryBase* Category, ...);
template CORE_API void BasicLog<ELogVerbosity::SetColor>(TStaticBasicLogRecordParam<TCHAR> Log, const FLogCategoryBase* Category, ...);

template CORE_API void BasicLog<ELogVerbosity::Error>(TStaticBasicLogRecordParam<ANSICHAR> Log, const FLogCategoryBase* Category, ...);
template CORE_API void BasicLog<ELogVerbosity::Warning>(TStaticBasicLogRecordParam<ANSICHAR> Log, const FLogCategoryBase* Category, ...);
template CORE_API void BasicLog<ELogVerbosity::Display>(TStaticBasicLogRecordParam<ANSICHAR> Log, const FLogCategoryBase* Category, ...);
template CORE_API void BasicLog<ELogVerbosity::Log>(TStaticBasicLogRecordParam<ANSICHAR> Log, const FLogCategoryBase* Category, ...);
template CORE_API void BasicLog<ELogVerbosity::Verbose>(TStaticBasicLogRecordParam<ANSICHAR> Log, const FLogCategoryBase* Category, ...);
template CORE_API void BasicLog<ELogVerbosity::VeryVerbose>(TStaticBasicLogRecordParam<ANSICHAR> Log, const FLogCategoryBase* Category, ...);
template CORE_API void BasicLog<ELogVerbosity::SetColor>(TStaticBasicLogRecordParam<ANSICHAR> Log, const FLogCategoryBase* Category, ...);

template CORE_API void BasicLog<ELogVerbosity::Error>(TStaticBasicLogRecordParam<UTF8CHAR> Log, const FLogCategoryBase* Category, ...);
template CORE_API void BasicLog<ELogVerbosity::Warning>(TStaticBasicLogRecordParam<UTF8CHAR> Log, const FLogCategoryBase* Category, ...);
template CORE_API void BasicLog<ELogVerbosity::Display>(TStaticBasicLogRecordParam<UTF8CHAR> Log, const FLogCategoryBase* Category, ...);
template CORE_API void BasicLog<ELogVerbosity::Log>(TStaticBasicLogRecordParam<UTF8CHAR> Log, const FLogCategoryBase* Category, ...);
template CORE_API void BasicLog<ELogVerbosity::Verbose>(TStaticBasicLogRecordParam<UTF8CHAR> Log, const FLogCategoryBase* Category, ...);
template CORE_API void BasicLog<ELogVerbosity::VeryVerbose>(TStaticBasicLogRecordParam<UTF8CHAR> Log, const FLogCategoryBase* Category, ...);
template CORE_API void BasicLog<ELogVerbosity::SetColor>(TStaticBasicLogRecordParam<UTF8CHAR> Log, const FLogCategoryBase* Category, ...);

template <UE::CCharType FormatCharType>
UE_AUTORTFM_ALWAYS_OPEN
static void BasicFatalLogV(TStaticBasicLogRecordParam<FormatCharType> Log, void* ProgramCounter, va_list Args)
{
#if !NO_LOGGING
	StaticFailDebugV(TEXT("Fatal error:"), Log.File, Log.Line, ProgramCounter, Log.Format, Args);
	UE_DEBUG_BREAK_AND_PROMPT_FOR_REMOTE();
	FDebug::ProcessFatalError(ProgramCounter);
#endif
}

template <UE::CCharType FormatCharType>
FORCENOINLINE void BasicFatalLog(TStaticBasicLogRecordParam<FormatCharType> Log, const FLogCategoryBase* Category, ...)
{
#if !NO_LOGGING
	va_list Args;
	va_start(Args, Category);
	BasicFatalLogV(Log, PLATFORM_RETURN_ADDRESS(), Args);
	va_end(Args);
#endif
}

template CORE_API void BasicFatalLog(TStaticBasicLogRecordParam<TCHAR> Log, const FLogCategoryBase* Category, ...);
template CORE_API void BasicFatalLog(TStaticBasicLogRecordParam<ANSICHAR> Log, const FLogCategoryBase* Category, ...);
template CORE_API void BasicFatalLog(TStaticBasicLogRecordParam<UTF8CHAR> Log, const FLogCategoryBase* Category, ...);

template <UE::CCharType FormatCharType>
FORCENOINLINE void BasicFatalLogWithProgramCounter(TStaticBasicLogRecordParam<FormatCharType> Log, const FLogCategoryBase* Category, void* ProgramCounter, ...)
{
#if !NO_LOGGING
	va_list Args;
	va_start(Args, ProgramCounter);
	BasicFatalLogV(Log, ProgramCounter, Args);
	va_end(Args);
#endif
}

template CORE_API void BasicFatalLogWithProgramCounter(TStaticBasicLogRecordParam<TCHAR> Log, const FLogCategoryBase* Category, void* ProgramCounter, ...);
template CORE_API void BasicFatalLogWithProgramCounter(TStaticBasicLogRecordParam<ANSICHAR> Log, const FLogCategoryBase* Category, void* ProgramCounter, ...);
template CORE_API void BasicFatalLogWithProgramCounter(TStaticBasicLogRecordParam<UTF8CHAR> Log, const FLogCategoryBase* Category, void* ProgramCounter, ...);

#if !NO_LOGGING
static constexpr const UTF8CHAR* const GDebugMessageLogFormat = UTF8TEXT("{Prefix}{Message}");

static FLogTemplate* GetDebugMessageLogTemplate()
{
	static FInlineLogTemplate Template(GDebugMessageLogFormat);
	return Template.Get();
}

// Serializing the log to compact binary happens in its own function because that allows stack
// space for the writer to be returned before calling into the output devices.
FORCENOINLINE static FCbObject SerializeDebugLogMessage(const TCHAR* Prefix, const TCHAR* Message)
{
	TCbWriter<512> Writer;
	Writer.BeginObject();

	const FAnsiStringView PrefixName = ANSITEXTVIEW("Prefix");
	Writer.AddString(PrefixName, Prefix);
	const FAnsiStringView MessageName = ANSITEXTVIEW("Message");
	Writer.AddString(MessageName, Message);

	AddContextsToBasicLog(Writer, MessageName, PrefixName);

	Writer.EndObject();
	return Writer.Save().AsObject();
}
#endif // !NO_LOGGING

UE_AUTORTFM_ALWAYS_OPEN
void LogDebugMessageLine(const ANSICHAR* File, int32 Line, const FName& InLogName, ELogVerbosity::Type Verbosity, const TCHAR* Message, const TCHAR* Prefix)
{
#if !NO_LOGGING
	FCbObject Fields = SerializeDebugLogMessage(Prefix, Message);

	FLogRecord Record;
	Record.SetUtf8Format(GDebugMessageLogFormat);
	Record.SetTemplate(GetDebugMessageLogTemplate());
	Record.SetFields(MoveTemp(Fields));
	Record.SetFile(File);
	Record.SetLine(Line);
	Record.SetCategory(InLogName);
	Record.SetVerbosity(Verbosity);
	Record.SetTime(FLogTime::Now());

	DispatchLogRecord(Record);
#endif // !NO_LOGGING
}
} // UE::Logging::Private

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

namespace UE
{

void DispatchDynamicLogRecord(const FLogRecord& Record)
{
	Logging::Private::DispatchLogRecord(Record);
}

void VisitLogContext(TFunctionRef<void (const FCbField&)> Visitor)
{
	using namespace UE::Logging::Private;
	for (const FLogContext* Node = LogContextHead; Node; Node = Node->Next)
	{
		if (!Node->bDisabledByNewerContext)
		{
			TCbWriter<256> Writer;
			Writer.AddField(Node->Name, FCbFieldView(Node->Value.GetData()));
			Visitor(Writer.Save());
		}
	}
}

} // UE
