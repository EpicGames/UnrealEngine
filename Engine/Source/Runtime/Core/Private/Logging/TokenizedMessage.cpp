// Copyright Epic Games, Inc. All Rights Reserved.

#include "Logging/TokenizedMessage.h"

#include "CoreTypes.h"
#include "Delegates/DelegateInstancesImpl.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformProcess.h"
#include "Internationalization/Internationalization.h"

#define LOCTEXT_NAMESPACE "Core.MessageLog"

TSharedRef<FTokenizedMessage> FTokenizedMessage::Create(EMessageSeverity::Type InSeverity, const FText& InMessageText)
{
	TSharedRef<FTokenizedMessage> Message = MakeShared<FTokenizedMessage>(FPrivateToken());
	Message->SetSeverity( InSeverity );
	Message->AddToken( FSeverityToken::Create(InSeverity) );
	if(!InMessageText.IsEmpty())
	{
		Message->AddToken( FTextToken::Create(InMessageText) );
	}
	return Message;
}

TSharedRef<FTokenizedMessage> FTokenizedMessage::Clone() const
{
	TSharedRef<FTokenizedMessage> Message = MakeShared<FTokenizedMessage>(FPrivateToken());
	*Message = *this;
	return Message;
}

FText FTokenizedMessage::ToText() const
{
	FText OutMessage;
	int32 TokenIndex = 0;

	// Don't duplicate starting severity when displaying as string - but display it if it differs (for whatever reason)
	if(MessageTokens.Num() > 0 && MessageTokens[0]->GetType() == EMessageToken::Severity)
	{
		TSharedRef<FSeverityToken> SeverityToken = StaticCastSharedRef<FSeverityToken>(MessageTokens[0]);
		if(SeverityToken->GetSeverity() != Severity)
		{
			OutMessage = FText::Format(LOCTEXT("SeverityMessageTokenFormatter", "{0}:"), FTokenizedMessage::GetSeverityText(Severity));
		}

		// Skip the first token message as the Severity gets added again in FMsg::Logf
		TokenIndex = 1;
	}
	else
	{
		OutMessage = FTokenizedMessage::GetSeverityText( Severity );
	}

	//@todo This is bad and not safe for localization, this needs to be refactored once rich text is implemented [9/24/2013 justin.sargent]
	for(; TokenIndex < MessageTokens.Num(); TokenIndex++)
	{
		if ( !OutMessage.IsEmpty() )
		{
			OutMessage = FText::Format( LOCTEXT("AggregateMessageTokenFormatter", "{0} {1}"), OutMessage, MessageTokens[TokenIndex]->ToText() );
		}
		else
		{
			OutMessage = MessageTokens[TokenIndex]->ToText();
		}
	}

	return OutMessage;
}

FText FTokenizedMessage::GetSeverityText( EMessageSeverity::Type InSeverity )
{
	switch (InSeverity)
	{
	case EMessageSeverity::Error:				
		return LOCTEXT("Error", "Error");						
	case EMessageSeverity::PerformanceWarning:	
		return LOCTEXT("PerfWarning", "Performance Warning");	
	case EMessageSeverity::Warning:				
		return LOCTEXT("Warning", "Warning");					
	case EMessageSeverity::Info:				
		return LOCTEXT("Info", "Info");							
	default:		
		return FText::GetEmpty();					
	}
}

FName FTokenizedMessage::GetSeverityIconName(EMessageSeverity::Type InSeverity)
{
	FName SeverityIconName;
	switch (InSeverity)
	{
	case EMessageSeverity::Error:				
		SeverityIconName = "Icons.ErrorWithColor";		
		break;
	case EMessageSeverity::PerformanceWarning:	
		SeverityIconName = "Icons.WarningWithColor";	
		break;
	case EMessageSeverity::Warning:				
		SeverityIconName = "Icons.WarningWithColor";	
		break;
	case EMessageSeverity::Info:				
		SeverityIconName = "Icons.BulletPoint";		
		break;
	default:		
		/* No icon for this type */						
		break;
	}
	return SeverityIconName;
}

FName FTokenizedMessage::GetIdentifier() const
{
	return Identifier;
}

void FTokenizedMessage::SetIdentifier(FName InIdentifier)
{
	Identifier = InIdentifier;
}

int32 FTokenizedMessage::GetIndentationLevel() const
{
	return IndentationLevel;
}

void FTokenizedMessage::SetIndentationLevel(int32 Level)
{
	IndentationLevel = FMath::Max(Level, 0);
}

TSharedRef<FTokenizedMessage> FTokenizedMessage::AddToken( const TSharedRef<IMessageToken>& InToken )
{
	MessageTokens.Add( InToken );
	return AsShared();
}

TSharedRef<FTokenizedMessage> FTokenizedMessage::AddText(const FText& InText)
{
	MessageTokens.Add(FTextToken::Create(InText));
	return AsShared();
}


void FTokenizedMessage::SetMessageLink(const TSharedRef<IMessageToken>& InToken)
{
	MessageLink = InToken;
}

void FTokenizedMessage::SetSeverity( const EMessageSeverity::Type InSeverity )
{
	if (Severity != InSeverity)
	{
		if (MessageTokens.Num() > 0 && MessageTokens[0]->GetType() == EMessageToken::Severity)
		{
			if (TSharedRef<FSeverityToken> SeverityToken = StaticCastSharedRef<FSeverityToken>(MessageTokens[0]);
				SeverityToken->GetSeverity() == Severity)
			{
				MessageTokens[0] = FSeverityToken::Create(InSeverity);
			}
		}
		Severity = InSeverity;
	}
}

EMessageSeverity::Type FTokenizedMessage::GetSeverity() const
{
	return Severity;
}

const TArray<TSharedRef<IMessageToken> >& FTokenizedMessage::GetMessageTokens() const
{
	return MessageTokens;
}

TSharedPtr<IMessageToken> FTokenizedMessage::GetMessageLink() const
{
	return MessageLink;
}

TSharedRef<FTextToken> FTextToken::Create(const FText& InMessage, bool InIsSourceLinkOnLeft)
{
	return MakeShared<FTextToken>(FPrivateToken(), InMessage, InIsSourceLinkOnLeft);
}

TSharedRef<FDynamicTextToken> FDynamicTextToken::Create(const TAttribute<FText>& InMessage)
{
	return MakeShared<FDynamicTextToken>(FPrivateToken(), InMessage);
}

TSharedRef<FImageToken> FImageToken::Create(const FName& InImageName)
{
	return MakeShared<FImageToken>(FPrivateToken(), InImageName);
}

TSharedRef<FSeverityToken> FSeverityToken::Create(EMessageSeverity::Type InSeverity)
{
	return MakeShared<FSeverityToken>(FPrivateToken(), InSeverity);
}

TSharedRef<FURLToken> FURLToken::Create(const FString& InURL, const FText& InMessage)
{
	return MakeShared<FURLToken>(FPrivateToken(), InURL, InMessage);
}

void FURLToken::VisitURL(const TSharedRef<IMessageToken>& Token, FString InURL)
{	
	FPlatformProcess::LaunchURL(*InURL, NULL, NULL);
}

FURLToken::FURLToken(const FString& InURL, const FText& InMessage)
{
	URL = InURL;

	if ( !InMessage.IsEmpty() )
	{
		CachedText = InMessage;
	}
	else
	{
		CachedText = NSLOCTEXT("Core.MessageLog", "DefaultHelpURLLabel", "Help");
	}

	MessageTokenActivated = FOnMessageTokenActivated::CreateStatic(&FURLToken::VisitURL, URL);
}

FAssetNameToken::FOnGotoAsset FAssetNameToken::GotoAsset;

void FAssetNameToken::FindAsset(const TSharedRef<IMessageToken>& Token, FString InAssetName)
{
	if(GotoAsset.IsBound())
	{
		GotoAsset.Execute(InAssetName);
	}
}

TSharedRef<FAssetNameToken> FAssetNameToken::Create(const FString& InAssetName, const FText& InMessage)
{
	return MakeShared<FAssetNameToken>(FPrivateToken(), InAssetName, InMessage);
}

FAssetNameToken::FAssetNameToken(const FString& InAssetName, const FText& InMessage)
	: AssetName(InAssetName)
{
	if ( !InMessage.IsEmpty() )
	{
		CachedText = InMessage;
	}
	else
	{
		CachedText = FText::FromString( AssetName );
	}

	MessageTokenActivated = FOnMessageTokenActivated::CreateStatic(&FAssetNameToken::FindAsset, AssetName);
}

FDocumentationToken::FDocumentationToken(const FString& InDocumentationLink, const FString& InPreviewExcerptLink, const FString& InPreviewExcerptName)
	: DocumentationLink(InDocumentationLink)
	, PreviewExcerptLink(InPreviewExcerptLink)
	, PreviewExcerptName(InPreviewExcerptName)
{
	if (!PreviewExcerptName.IsEmpty())
	{
		DocumentationLink = DocumentationLink + "#" + PreviewExcerptName.ToLower();
	}
}

TSharedRef<FActionToken> FActionToken::Create(const FText& InActionName, const FText& InActionDescription, const FOnActionTokenExecuted& InAction, bool bInSingleUse)
{
	return MakeShared<FActionToken>(FPrivateToken(), InActionName, InActionDescription, InAction, bInSingleUse);
}

TSharedRef<FActionToken> FActionToken::Create(const FText& InActionName, const FText& InActionDescription, const FOnActionTokenExecuted& InAction, const FCanExecuteActionToken& InCanExecuteAction, bool bInSingleUse)
{
	return MakeShared<FActionToken>(FPrivateToken(), InActionName, InActionDescription, InAction, InCanExecuteAction, bInSingleUse);
}

TSharedRef<FTutorialToken> FTutorialToken::Create(const FString& TutorialAssetName)
{
	return MakeShared<FTutorialToken>(FPrivateToken(), TutorialAssetName);
}

TSharedRef<FDocumentationToken> FDocumentationToken::Create(const FString& InDocumentationLink, const FString& InPreviewExcerptLink, const FString& InPreviewExcerptName)
{
	return MakeShared<FDocumentationToken>(FPrivateToken(), InDocumentationLink, InPreviewExcerptLink, InPreviewExcerptName);
}

FOnMessageTokenActivated FActorToken::DefaultMessageTokenActivated;

TSharedRef<FActorToken> FActorToken::Create(const FString& InActorPath, const FGuid& InActorGuid, const FText& InMessage)
{
	return MakeShared<FActorToken>(FPrivateToken(), InActorPath, InActorGuid, InMessage);
}

FActorToken::FActorToken(const FString& InActorPath, const FGuid& InActorGuid, const FText& InMessage)
	: ActorPath(InActorPath), ActorGuid(InActorGuid)
{
	if (!InMessage.IsEmpty())
	{
		CachedText = InMessage;
	}
	else
	{
		CachedText = FText::FromString(ActorPath);
	}
}

const FOnMessageTokenActivated& FActorToken::GetOnMessageTokenActivated() const
{
	if (MessageTokenActivated.IsBound())
	{
		return MessageTokenActivated;
	}
	else
	{
		return DefaultMessageTokenActivated;
	}
}

TSharedRef<FFixToken> FFixToken::Create(const FText& InName, TSharedRef<UE::DataValidation::IFixer> InFixer, int32 InFixIndex)
{
	return MakeShared<FFixToken>(FPrivateToken(), InName, MoveTemp(InFixer), InFixIndex);
}

FFixToken::FFixToken(const FText& InName, TSharedPtr<UE::DataValidation::IFixer> InFixer, int32 InFixIndex)
	: Fixer(MoveTemp(InFixer))
	, FixIndex(InFixIndex)
{
	CachedText = InName.IsEmpty() ? LOCTEXT("FixLabelEmpty", "Fix Issue") : InName;
}

#undef LOCTEXT_NAMESPACE
