// Copyright Epic Games, Inc. All Rights Reserved.

#include "Mac/CocoaThread.h"
#include "Apple/ScopeAutoreleasePool.h"

#include "Containers/LockFreeList.h"
#include "HAL/IConsoleManager.h"
#include "Templates/RefCounting.h"
#include "Misc/OutputDeviceRedirector.h"

// this is the size of the game thread stack, it must be a multiple of 4k
#if (UE_BUILD_DEBUG)
	#define GAME_THREAD_STACK_SIZE 64 * 1024 * 1024
#else
	#define GAME_THREAD_STACK_SIZE 128 * 1024 * 1024
#endif

NSString* UnrealNilEventMode = @"UnrealNilEventMode";
NSString* UnrealShowEventMode = @"UnrealShowEventMode";
NSString* UnrealResizeEventMode = @"UnrealResizeEventMode";
NSString* UnrealFullscreenEventMode = @"UnrealFullscreenEventMode";
NSString* UnrealCloseEventMode = @"UnrealCloseEventMode";
NSString* UnrealIMEEventMode = @"UnrealIMEEventMode";

static FCocoaGameThread* GCocoaGameThread = nil;
uint32 GMacMainThreadId = 0;

static TAutoConsoleVariable<bool> CVarMacCocoaThreadIgnoreWaitMode(
	TEXT("Mac.IgnoreCocoaThreadWaitMode"),
	1,
	TEXT("Wether or not CocoaThread should ignore the specified WaitMode when processing blocks.\n")
    TEXT("Changing the value to 0 might cause deadlocks because it will prevent the thread that calls MainThreadCall from processing certain types of blocks.\n")
	TEXT("Default is 1."),
	ECVF_ReadOnly);

class FCocoaRunLoopSource;
@interface FCocoaRunLoopSourceInfo : NSObject
{
@private
	FCocoaRunLoopSource* Source;
	CFRunLoopRef RunLoop;
	CFStringRef Mode;
}
- (id)initWithSource:(FCocoaRunLoopSource*)InSource;
- (void)dealloc;
- (void)scheduleOn:(CFRunLoopRef)InRunLoop inMode:(CFStringRef)InMode;
- (void)cancelFrom:(CFRunLoopRef)InRunLoop inMode:(CFStringRef)InMode;
- (void)perform;
@end

class FCocoaRunLoopTask
{
	friend class FCocoaRunLoopSource;
public:
	FCocoaRunLoopTask(dispatch_block_t InBlock, NSArray* InModes)
	: Block(Block_copy(InBlock))
	, Modes([InModes retain])
	{
		
	}
	
	~FCocoaRunLoopTask()
	{
		[Modes release];
		Block_release(Block);
	}
private:
	dispatch_block_t Block;
	NSArray* Modes;
};

class FCocoaRunLoopSource : public FRefCountedObject
{
public:
	static void RegisterMainRunLoop(CFRunLoopRef RunLoop)
	{
		check(!MainRunLoopSource);
		MainRunLoopSource = new FCocoaRunLoopSource(RunLoop);
	}
	
	static void RegisterGameRunLoop(CFRunLoopRef RunLoop)
	{
		check(!GameRunLoopSource);
		GameRunLoopSource = new FCocoaRunLoopSource(RunLoop);
	}
	
	static FCocoaRunLoopSource& GetMainRunLoopSource(void)
	{
		check(MainRunLoopSource);
		return *MainRunLoopSource;
	}
	
	static FCocoaRunLoopSource& GetGameRunLoopSource(void)
	{
		check(GameRunLoopSource);
		return *GameRunLoopSource;
	}
	
	void Schedule(dispatch_block_t InBlock, NSArray* InModes)
	{
		for (NSString* Mode in InModes)
		{
			Register((CFStringRef)Mode);
		}
		
		Tasks.Push(new FCocoaRunLoopTask(InBlock, InModes));
		
		CFDictionaryApplyFunction(SourceDictionary, &FCocoaRunLoopSource::SignalFunction, nullptr);
	}
	
	void Wake(void)
	{
		CFRunLoopWakeUp(TargetRunLoop);
	}
	
	void Process(CFStringRef Mode)
	{
		Tasks.PopAll(OutstandingTasks);
		
		bool bDone = false;
		while (!bDone)
		{
			bDone = true;
			for (uint32 i = 0; i < OutstandingTasks.Num(); i++)
			{
				FCocoaRunLoopTask* Task = OutstandingTasks[i];
				if (Task && [Task->Modes containsObject:(NSString*)Mode])
				{
					OutstandingTasks.RemoveAt(i);
					Task->Block();
					delete Task;
					bDone = false;
					break;
				}
			}
		}
	}
	
private:
	FCocoaRunLoopSource(CFRunLoopRef RunLoop)
	: TargetRunLoop(RunLoop)
	, SourceDictionary(CFDictionaryCreateMutable(nullptr, 0, &kCFTypeDictionaryKeyCallBacks, &kCFTypeDictionaryValueCallBacks))
	{
		check(TargetRunLoop);
		
		// Register for default modes
		Register(kCFRunLoopDefaultMode);
		Register((CFStringRef)NSModalPanelRunLoopMode);
		Register((CFStringRef)UnrealNilEventMode);
		Register((CFStringRef)UnrealShowEventMode);
		Register((CFStringRef)UnrealResizeEventMode);
		Register((CFStringRef)UnrealFullscreenEventMode);
		Register((CFStringRef)UnrealCloseEventMode);
		Register((CFStringRef)UnrealIMEEventMode);
	}
	
	virtual ~FCocoaRunLoopSource()
	{
		if ( MainRunLoopSource == this )
		{
			MainRunLoopSource = nullptr;
		}
		else if ( GameRunLoopSource == this )
		{
			GameRunLoopSource = nullptr;
		}
		
		CFDictionaryApplyFunction(SourceDictionary, &FCocoaRunLoopSource::ShutdownFunction, TargetRunLoop);
		CFRelease(SourceDictionary);
	}
	
	void Register(CFStringRef Mode)
	{
		if(CFDictionaryContainsKey(SourceDictionary, Mode) == false)
		{
			CFRunLoopSourceContext Context;
			FCocoaRunLoopSourceInfo* Info = [[FCocoaRunLoopSourceInfo alloc] initWithSource:this];
			FMemory::Memzero(Context);
			Context.version = 0;
			Context.info = Info;
			Context.retain = CFRetain;
			Context.release = CFRelease;
			Context.copyDescription = CFCopyDescription;
			Context.equal = CFEqual;
			Context.hash = CFHash;
			Context.schedule = &FCocoaRunLoopSource::Schedule;
			Context.cancel = &FCocoaRunLoopSource::Cancel;
			Context.perform = &FCocoaRunLoopSource::Perform;
			
			CFRunLoopSourceRef Source = CFRunLoopSourceCreate(nullptr, 0, &Context);
			CFDictionaryAddValue(SourceDictionary, Mode, Source);
			CFRunLoopAddSource(TargetRunLoop, Source, Mode);
			CFRelease(Source);
		}
	}
	
	static void SignalFunction(const void* Key, const void* Value, void* Context)
	{
		CFRunLoopSourceRef RunLoopSource = (CFRunLoopSourceRef)Value;
		if(RunLoopSource)
		{
			CFRunLoopSourceSignal(RunLoopSource);
		}
	}
	static void ShutdownFunction(const void* Key, const void* Value, void* Context)
	{
		CFRunLoopRef RunLoop = static_cast<CFRunLoopRef>(Context);
		if(RunLoop)
		{
			CFRunLoopRemoveSource(RunLoop, (CFRunLoopSourceRef)Value, (CFStringRef)Key);
		}
	}
	static void Schedule(void* Info, CFRunLoopRef RunLoop, CFStringRef Mode)
	{
		FCocoaRunLoopSourceInfo* Source = static_cast<FCocoaRunLoopSourceInfo*>(Info);
		if (Source)
		{
			[Source scheduleOn:RunLoop inMode:Mode];
		}
	}
    static void Cancel(void* Info, CFRunLoopRef RunLoop, CFStringRef Mode)
	{
		FCocoaRunLoopSourceInfo* Source = static_cast<FCocoaRunLoopSourceInfo*>(Info);
		if (Source)
		{
			[Source cancelFrom:RunLoop inMode:Mode];
		}
	}
    static void Perform(void* Info)
	{
		FCocoaRunLoopSourceInfo* Source = static_cast<FCocoaRunLoopSourceInfo*>(Info);
		if (Source)
		{
			[Source perform];
		}
	}
	
private:
	TLockFreePointerListFIFO<FCocoaRunLoopTask, 0> Tasks;
	TArray<FCocoaRunLoopTask*> OutstandingTasks;
	CFRunLoopRef TargetRunLoop;
	CFMutableDictionaryRef SourceDictionary;
	
private:
	static FCocoaRunLoopSource* MainRunLoopSource;
	static FCocoaRunLoopSource* GameRunLoopSource;
};

FCocoaRunLoopSource* FCocoaRunLoopSource::MainRunLoopSource = nullptr;
FCocoaRunLoopSource* FCocoaRunLoopSource::GameRunLoopSource = nullptr;

@implementation FCocoaRunLoopSourceInfo

- (id)initWithSource:(FCocoaRunLoopSource*)InSource
{
	id Self = [super init];
	if(Self)
	{
		check(InSource);
		Source = InSource;
		Source->AddRef();
	}
	return Self;
}

- (void)dealloc
{
	check(Source);
	Source->Release();
	Source = nullptr;
	[super dealloc];
}

- (void)scheduleOn:(CFRunLoopRef)InRunLoop inMode:(CFStringRef)InMode
{
	check(!RunLoop);
	check(!Mode);
	RunLoop = InRunLoop;
	Mode = InMode;
}

- (void)cancelFrom:(CFRunLoopRef)InRunLoop inMode:(CFStringRef)InMode
{
	if(CFEqual(InRunLoop, RunLoop) && CFEqual(Mode, InMode))
	{
		RunLoop = nullptr;
		Mode = nullptr;
	}
}

- (void)perform
{
	check(Source);
	check(RunLoop);
	check(Mode);
	check(CFEqual(RunLoop, CFRunLoopGetCurrent()));
	CFStringRef CurrentMode = CFRunLoopCopyCurrentMode(CFRunLoopGetCurrent());
	check(CFEqual(CurrentMode, Mode));
	Source->Process(CurrentMode);
	CFRelease(CurrentMode);
}

@end

@implementation NSThread (FCocoaThread)

+ (NSThread*) gameThread
{
	if(!GCocoaGameThread)
	{
		return [NSThread mainThread];
	}
	return GCocoaGameThread;
}

+ (bool) isGameThread
{
	bool const bIsGameThread = [[NSThread currentThread] isGameThread];
	return bIsGameThread;
}

- (bool) isGameThread
{
	bool const bIsGameThread = (self == GCocoaGameThread);
	return bIsGameThread;
}

@end

@implementation FCocoaGameThread : NSThread

- (id)init
{
	id Self = [super init];
	if(Self)
	{
		GCocoaGameThread = Self;
	}
	return Self;
}

- (id)initWithTarget:(id)Target selector:(SEL)Selector object:(id)Argument
{
	id Self = [super initWithTarget:Target selector:Selector object:Argument];
	if(Self)
	{
		GCocoaGameThread = Self;
	}
	return Self;
}

- (void)dealloc
{
	GCocoaGameThread = nil;
	[super dealloc];
}

- (void)main
{
	struct sched_param Sched;
	FMemory::Memzero(&Sched, sizeof(struct sched_param));
	int32 Policy = SCHED_RR; // It may be that Mac would also benefit from FIFO for Game, Render & RHI thread scheduling
	
	// Read the current policy
	pthread_getschedparam(pthread_self(), &Policy, &Sched);
	
	// set the priority appropriately
	Sched.sched_priority = 45; // Equivalent to TPri_Highest
	pthread_setschedparam(pthread_self(), Policy, &Sched);
	
	NSRunLoop* GameRunLoop = [NSRunLoop currentRunLoop];
	
	// Register game run loop source
	FCocoaRunLoopSource::RegisterGameRunLoop([GameRunLoop getCFRunLoop]);
	
	if (GLog)
	{
		GLog->SetCurrentThreadAsPrimaryThread();
	}

	[super main];
	
	// We have exited the game thread, so any UE code running now should treat the Main thread
	// as the game thread, so we don't crash in static destructors.
	GGameThreadId = GMacMainThreadId;
	GCocoaGameThread = nil;
	
	// Tell the main thread we are OK to quit, but don't wait for it.
	if (IsEngineExitRequested())
	{
		MainThreadCall(^{
			[NSApp replyToApplicationShouldTerminate:YES];
			[[NSProcessInfo processInfo] enableSuddenTermination];
		}, false);
	}
	else
	{
		MainThreadCall(^{
			[[NSProcessInfo processInfo] enableSuddenTermination];
		}, false);
	}
	
	// And now it is time to die.
	[self release];
}

@end

// Store the WaitMode of the thread that scheduled the execution block on the thread that is going to process it
thread_local NSString* GThreadInstigatorWaitMode = nullptr;

static void PerformBlockOnThread(FCocoaRunLoopSource& ThreadSource, dispatch_block_t Block, NSArray* SendModes, NSString* WaitMode, bool const bWait)
{
	
	if ( bWait )
	{
		// We need to copy this block because it's going to be within the block in the task.
		dispatch_block_t CopiedBlock = Block_copy(Block);
		dispatch_semaphore_t Semaphore = dispatch_semaphore_create(0);
		NSString* ThreadInstigatorWaitMode = [WaitMode retain];
		dispatch_block_t ExecuteBlock = ^{
			NSString* SavedInstigatorWaitMode = GThreadInstigatorWaitMode;			
			GThreadInstigatorWaitMode = ThreadInstigatorWaitMode;
			CopiedBlock();
			if (GThreadInstigatorWaitMode)
			{
				[GThreadInstigatorWaitMode release];
			}
			GThreadInstigatorWaitMode = SavedInstigatorWaitMode;
			dispatch_semaphore_signal(Semaphore);
		};
		
		// If the thread instigator wait mode is not already contained in the SendModes, we add it to prevent potential deadlocks.
		// The idea is that if thread A schedule a block on thread B and while executing that block thread B has to schedule a block on thread A, that block can be executed.
		if (GThreadInstigatorWaitMode && ![SendModes containsObject:GThreadInstigatorWaitMode])
		{
			ThreadSource.Schedule(ExecuteBlock, [SendModes arrayByAddingObject:GThreadInstigatorWaitMode]);
		}
		else
		{
			ThreadSource.Schedule(ExecuteBlock, SendModes);
		}
		
		do {
			ThreadSource.Wake();
			CFRunLoopRunInMode((CFStringRef)WaitMode, 0, true);
		} while (dispatch_semaphore_wait(Semaphore, dispatch_time(0, 100000ull)));
		
		Block_release(CopiedBlock);
		dispatch_release(Semaphore);
	}
	else
	{
		ThreadSource.Schedule(Block, SendModes);
		ThreadSource.Wake();
	}
}

void MainThreadCall(dispatch_block_t Block, bool const bWait, NSString* WaitMode)
{
	if ( [NSThread mainThread] != [NSThread currentThread] )
	{
		if (bWait && CVarMacCocoaThreadIgnoreWaitMode.GetValueOnAnyThread())
		{
			WaitMode = NSDefaultRunLoopMode;
		}
		FCocoaRunLoopSource& MainRunLoopSource = FCocoaRunLoopSource::GetMainRunLoopSource();
		PerformBlockOnThread(MainRunLoopSource, Block, @[ NSDefaultRunLoopMode, NSModalPanelRunLoopMode, NSEventTrackingRunLoopMode ], WaitMode, bWait);
	}
	else
	{
		Block();
	}
}

void GameThreadCall(dispatch_block_t Block, bool const bWait, NSArray* SendModes)
{
	if ( [NSThread gameThread] != [NSThread currentThread] )
	{
		FCocoaRunLoopSource& GameRunLoopSource = FCocoaRunLoopSource::GetGameRunLoopSource();
		PerformBlockOnThread(GameRunLoopSource, Block, SendModes, NSDefaultRunLoopMode, bWait);
	}
	else
	{
		Block();
	}
}

void RegisterCurrentThreadAsMain()
{
	GMacMainThreadId = FPlatformTLS::GetCurrentThreadId();
#if MAC_SEPARATE_GAME_THREAD
	FCocoaRunLoopSource::RegisterMainRunLoop(CFRunLoopGetCurrent());
#endif
}

void RunGameThread(id Target, SEL Selector)
{
	SCOPED_AUTORELEASE_POOL;

	// @todo: Proper support for sudden termination (OS termination of application without any events, notifications or signals) - which presently can assert, crash or corrupt. At present we can't deal with this and we need to ensure that we don't permit the system to kill us without warning by disabling & enabling support around operations which must be committed to disk atomically.
	[[NSProcessInfo processInfo] disableSuddenTermination];

	RegisterCurrentThreadAsMain();

#if MAC_SEPARATE_GAME_THREAD
	// Create a separate game thread and set it to the stack size to be the same as the main thread default of 8MB ( http://developer.apple.com/library/mac/#qa/qa1419/_index.html )
	FCocoaGameThread* GameThread = [[FCocoaGameThread alloc] initWithTarget:Target selector:Selector object:nil];
	[GameThread setStackSize:GAME_THREAD_STACK_SIZE];
	[GameThread start];
	[GameThread setName:@"GameThread"];
#else
	[Target performSelector:Selector withObject:nil];

	if (IsEngineExitRequested())
	{
		[NSApp replyToApplicationShouldTerminate:YES];
	}
#endif
}

void ProcessGameThreadEvents(void)
{
	SCOPED_AUTORELEASE_POOL;
#if MAC_SEPARATE_GAME_THREAD
	// Make one pass through the loop, processing all events
	CFRunLoopRef RunLoop = CFRunLoopGetCurrent();
	CFRunLoopRunInMode(kCFRunLoopDefaultMode, 0, false);
#else
	while( NSEvent *Event = [NSApp nextEventMatchingMask: NSAnyEventMask untilDate: nil inMode: NSDefaultRunLoopMode dequeue: true] )
	{
		// Either the windowNumber is 0 or the window must be valid for the event to be processed.
		// Processing events with a windowNumber but no window will crash inside sendEvent as it will try to send to a destructed window.
		if ([Event windowNumber] == 0 || [Event window] != nil)
		{
			[NSApp sendEvent: Event];
		}
	}
#endif
}
