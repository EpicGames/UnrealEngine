// Copyright Epic Games, Inc. All Rights Reserved.

#include "Mac/MacMallocZone.h"
#include "Mac/MacPlatformMisc.h"
#include "Mac/MacPlatformCrashContext.h"
#include "HAL/PlatformTLS.h"
#include "HAL/PlatformProcess.h"
#include <malloc/malloc.h>

FMacMallocZone::FMacMallocZone( uint64 const InitialSize )
: MemoryZone( malloc_create_zone( InitialSize, 0 ) )
{
}

FMacMallocZone::~FMacMallocZone()
{
	if ( MemoryZone )
	{
		malloc_destroy_zone( MemoryZone );
	}
}

void* FMacMallocZone::Malloc( SIZE_T Size, uint32 Alignment )
{
	check( MemoryZone );
	void* Result = malloc_zone_malloc( MemoryZone, Size );
	return Result;
}

void* FMacMallocZone::Realloc( void* Ptr, SIZE_T NewSize, uint32 Alignment )
{
	check( MemoryZone );
	void* Result = malloc_zone_realloc( MemoryZone, Ptr, NewSize );
	return Result;
}

void FMacMallocZone::Free( void* Ptr )
{
	check( MemoryZone );
	malloc_zone_free( MemoryZone, Ptr );
}

bool FMacMallocZone::GetAllocationSize(void *Original, SIZE_T &SizeOut)
{
	SizeOut = malloc_size( Original );
	return SizeOut > 0;
}

bool FMacMallocZone::IsInternallyThreadSafe() const
{
	return true;
}

bool FMacMallocZone::ValidateHeap()
{
	check( MemoryZone );
	return malloc_zone_check( MemoryZone ) ;
}

const TCHAR* FMacMallocZone::GetDescriptiveName()
{
	return TEXT("MallocZone");
}

FMacMallocCrashHandler::FMacMallocCrashHandler( uint64 const InitialSize )
: FMacMallocZone( InitialSize )
, OriginalHeap(UE::Private::GMalloc)
, CrashContext( nullptr )
, ThreadId( -1 )
{
	check(UE::Private::GMalloc);
}

FMacMallocCrashHandler::~FMacMallocCrashHandler()
{
	// We crashed, so don't try and tidy the malloc zone
	if ( ThreadId != -1 )
	{
		MemoryZone = nullptr;
	}
}
	
void FMacMallocCrashHandler::Enable( FMacCrashContext* Context, uint32 CrashedThreadId )
{
	check(Context);
	CrashContext = Context;
	ThreadId = CrashedThreadId;
	OriginalHeap = UE::Private::GMalloc;
	if (PLATFORM_USES_FIXED_GMalloc_CLASS && GFixedMallocLocationPtr)
	{
		*GFixedMallocLocationPtr = this; // replace fixed allocator with crash allocator
	}
	UE::Private::GMalloc = this;
}
	
void* FMacMallocCrashHandler::Malloc( SIZE_T Size, uint32 Alignment )
{
	void* Result = nullptr;
	if ( IsOnCrashedThread() )
	{
		Result = FMacMallocZone::Malloc( Size, Alignment );
		if( !Result )
		{
			check(CrashContext);
			CrashContext->GenerateCrashInfoAndLaunchReporter();
		}
	}
	return Result;
}

void* FMacMallocCrashHandler::Realloc( void* Ptr, SIZE_T NewSize, uint32 Alignment )
{
	void* Result = nullptr;
	if ( IsOnCrashedThread() )
	{
		if ( Ptr == nullptr || MemoryZone->size( MemoryZone, Ptr ) > 0 )
		{
			Result = FMacMallocZone::Realloc( Ptr, NewSize, Alignment );
			if( NewSize && !Result )
			{
				check(CrashContext);
				CrashContext->GenerateCrashInfoAndLaunchReporter();
			}
		}
		else if ( NewSize )
		{
			if ( FCStringWide::Strcmp( OriginalHeap->GetDescriptiveName(), TEXT("ANSI") ) != 0 )
			{
				SIZE_T OldSize = 0;
				if ( OriginalHeap->GetAllocationSize( Ptr, OldSize ) )
				{
					Result = Malloc( NewSize, Alignment );
					if ( Result )
					{
						FMemory::Memcpy( Result, Ptr, FMath::Min( NewSize, OldSize ) );
					}
					else
					{
						check(CrashContext);
						CrashContext->GenerateCrashInfoAndLaunchReporter();
					}
				}
			}
			else // Can't safely handle this, so just report & exit
			{
				check(CrashContext);
				CrashContext->GenerateCrashInfoAndLaunchReporter();
			}
		}
	}
	return Result;
}

void FMacMallocCrashHandler::Free( void* Ptr )
{
	if ( IsOnCrashedThread() && Ptr && MemoryZone->size( MemoryZone, Ptr ) > 0 )
	{
		FMacMallocZone::Free( Ptr );
	}
}

bool FMacMallocCrashHandler::GetAllocationSize( void *Original, SIZE_T &SizeOut )
{
	SizeOut = 0;
	if ( IsOnCrashedThread() && Original )
	{
		SizeOut = MemoryZone->size( MemoryZone, Original );
	}
	return SizeOut > 0;
}

const TCHAR* FMacMallocCrashHandler::GetDescriptiveName()
{
	return TEXT("MallocCrashHandler");
}

bool FMacMallocCrashHandler::IsOnCrashedThread( void )
{
	// Suspend threads other than the crashed one to prevent serious memory errors.
	// Only the crashed thread can do anything meaningful from here anyway.
	if ( ThreadId == FPlatformTLS::GetCurrentThreadId() )
	{
		return true;
	}
	else
	{
		FPlatformProcess::SleepInfinite();
	}
}
