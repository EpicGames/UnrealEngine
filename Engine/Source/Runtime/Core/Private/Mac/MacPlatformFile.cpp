// Copyright Epic Games, Inc. All Rights Reserved.

#include "Mac/MacPlatformFile.h"

IPlatformFile& IPlatformFile::GetPlatformPhysical()
{
	static FApplePlatformFile MacPlatformSingleton;
	return MacPlatformSingleton;
}
