// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Apple/ApplePlatformManualResetEvent.h"

namespace UE { using FPlatformManualResetEvent = UE::HAL::Private::FApplePlatformManualResetEvent; }
