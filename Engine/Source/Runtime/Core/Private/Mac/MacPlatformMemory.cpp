// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MacPlatformMemory.cpp: Mac platform memory functions
=============================================================================*/

#include "Mac/MacPlatformMemory.h"
#include "Mac/MacSystemIncludes.h"
#include "HAL/PlatformMemory.h"
#include "HAL/MallocTBB.h"
#include "HAL/MallocAnsi.h"
#include "HAL/MallocMimalloc.h"
#include "HAL/MallocBinned.h"
#include "HAL/MallocBinned2.h"
#include "HAL/MallocBinned3.h"
#include "HAL/MallocStomp.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CoreStats.h"
#include "CoreGlobals.h"

#include <sys/param.h>
#include <sys/mount.h>
#include <sys/mman.h>
#include <sys/posix_shm.h>

#include <mach/vm_page_size.h>

extern "C"
{
	#include <crt_externs.h> // Needed for _NSGetArgc & _NSGetArgv
}

#if PLATFORM_MAC_X86
void* CFNetwork_CFAllocatorOperatorNew_Replacement(unsigned long Size, CFAllocatorRef Alloc)
{
	if (Alloc)
	{
		return CFAllocatorAllocate(Alloc, Size, 0);
	}
	else
	{
		return FMemory::Malloc(Size);
	}
}
#endif // PLATFORM_MAC_X86

FGenericPlatformMemoryStats::EMemoryPressureStatus FMacPlatformMemory::MemoryPressureStatus = FGenericPlatformMemoryStats::EMemoryPressureStatus::Unknown;

static bool HasArg(const char *Arg)
{
	if (_NSGetArgc() && _NSGetArgv())
	{
		int Argc = *_NSGetArgc();
		char** Argv = *_NSGetArgv();

		for (int i = 1; i < Argc; ++i)
		{
			if (FCStringAnsi::Stricmp(Argv[i], Arg) == 0)
			{
				return true;
			}
		}
	}

	return false;
}

FMalloc* FMacPlatformMemory::BaseAllocator()
{
	auto dispatch_source = dispatch_source_create(DISPATCH_SOURCE_TYPE_MEMORYPRESSURE, 0, DISPATCH_MEMORYPRESSURE_NORMAL, dispatch_get_main_queue());
	dispatch_source_set_event_handler(dispatch_source, ^{FMacPlatformMemory::MemoryPressureStatus = FGenericPlatformMemoryStats::EMemoryPressureStatus::Nominal;});
	dispatch_activate(dispatch_source);
	dispatch_retain(dispatch_source);
	dispatch_source = dispatch_source_create(DISPATCH_SOURCE_TYPE_MEMORYPRESSURE, 0, DISPATCH_MEMORYPRESSURE_WARN, dispatch_get_main_queue());
	dispatch_source_set_event_handler(dispatch_source, ^{FMacPlatformMemory::MemoryPressureStatus = FGenericPlatformMemoryStats::EMemoryPressureStatus::Warning;});
	dispatch_activate(dispatch_source);
	dispatch_retain(dispatch_source);
	dispatch_source = dispatch_source_create(DISPATCH_SOURCE_TYPE_MEMORYPRESSURE, 0, DISPATCH_MEMORYPRESSURE_CRITICAL, dispatch_get_main_queue());
	dispatch_source_set_event_handler(dispatch_source, ^{FMacPlatformMemory::MemoryPressureStatus = FGenericPlatformMemoryStats::EMemoryPressureStatus::Critical;});
	dispatch_activate(dispatch_source);
	dispatch_retain(dispatch_source);
	
	static FMalloc* Instance = nullptr;
	if (Instance != nullptr)
	{
		return Instance;
	}

	if (FORCE_ANSI_ALLOCATOR)
	{
		AllocatorToUse = EMemoryAllocatorToUse::Ansi;
	}
	else if ((WITH_EDITORONLY_DATA || IS_PROGRAM || USING_INSTRUMENTATION) && MIMALLOC_ENABLED)
	{
		// Make mimalloc a default allocator on Editor as it has a significantly better perf
		AllocatorToUse = EMemoryAllocatorToUse::Mimalloc;
	}
	else if ((WITH_EDITORONLY_DATA || IS_PROGRAM) && TBBMALLOC_ENABLED)
	{
		AllocatorToUse = EMemoryAllocatorToUse::TBB;
	}
	else if (USE_MALLOC_BINNED2)
	{
		AllocatorToUse = EMemoryAllocatorToUse::Binned2;
	}
	else if (USE_MALLOC_BINNED3)
	{
		AllocatorToUse = EMemoryAllocatorToUse::Binned3;
	}
	else
	{
		AllocatorToUse = EMemoryAllocatorToUse::Binned;
	}

	// Force ANSI malloc per user preference.
	if((getenv("UE4_FORCE_MALLOC_ANSI") != nullptr) || HasArg("-ansimalloc"))
	{
		AllocatorToUse = EMemoryAllocatorToUse::Ansi;
	}
#if WITH_MALLOC_STOMP
	else if (HasArg("-stompmalloc"))
	{
		AllocatorToUse = EMemoryAllocatorToUse::Stomp;
	}
#endif // WITH_MALLOC_STOMP
#if MIMALLOC_ENABLED
	else if (HasArg("-mimalloc"))
	{
		AllocatorToUse = EMemoryAllocatorToUse::Mimalloc;
	}
#endif // MIMALLOC_ENABLED

	// Force ANSI malloc with TSAN
#if defined(__has_feature)
	#if __has_feature(thread_sanitizer)
		AllocatorToUse = EMemoryAllocatorToUse::Ansi;
	#endif
#endif

	switch (AllocatorToUse)
	{
	case EMemoryAllocatorToUse::Ansi:
		Instance = new FMallocAnsi();
		break;
#if WITH_MALLOC_STOMP
	case EMemoryAllocatorToUse::Stomp:
		Instance = new FMallocStomp();
		break;
#endif
#if TBBMALLOC_ENABLED
	case EMemoryAllocatorToUse::TBB:
		Instance = new FMallocTBB();
		break;
#endif
#if MIMALLOC_ENABLED
	case EMemoryAllocatorToUse::Mimalloc:
		Instance = new FMallocMimalloc();
		break;
#endif
	case EMemoryAllocatorToUse::Binned2:
		Instance = new FMallocBinned2();
		break;

	case EMemoryAllocatorToUse::Binned3:
		Instance = new FMallocBinned3();
		break;

	default:	// intentional fall-through
	case EMemoryAllocatorToUse::Binned:
		// [RCL] 2017-03-06 FIXME: perhaps BinnedPageSize should be used here, but leaving this change to the Mac platform owner.
		Instance = new FMallocBinned((uint32)(GetConstants().PageSize&MAX_uint32), 0x100000000);
		break;
	}

	return Instance;
}

FPlatformMemoryStats FMacPlatformMemory::GetStats()
{
	const FPlatformMemoryConstants& MemoryConstants = FPlatformMemory::GetConstants();

	static FPlatformMemoryStats MemoryStats{};

	// Gather platform memory stats.
	vm_statistics64 Stats;
	mach_msg_type_number_t StatsSize = HOST_VM_INFO64_COUNT;
	if (host_statistics64(mach_host_self(), HOST_VM_INFO64, (host_info_t)&Stats, &StatsSize) != KERN_SUCCESS)
	{
		UE_LOGF(LogTemp, Warning, "Failed to fetch vm statistics");
		return MemoryStats;
	}
	uint64_t FreeMem = (Stats.free_count + Stats.inactive_count) * MemoryConstants.PageSize;
	MemoryStats.AvailablePhysical = FreeMem;
	
	// Get swap file info
	xsw_usage SwapUsage;
	SIZE_T Size = sizeof(SwapUsage);
	sysctlbyname("vm.swapusage", &SwapUsage, &Size, NULL, 0);
	MemoryStats.AvailableVirtual = FreeMem + SwapUsage.xsu_avail;

	// Just get memory information for the process and report the working set instead
	task_vm_info TaskInfo;
	mach_msg_type_number_t TaskInfoCount = TASK_VM_INFO_COUNT;
	if (task_info( mach_task_self(), TASK_VM_INFO, (task_info_t)&TaskInfo, &TaskInfoCount ) != KERN_SUCCESS)
	{
		UE_LOGF(LogTemp, Warning, "Failed to fetch vm statistics");
		return MemoryStats;
	}
	
	// we want to use phys_footprint instead of resident size as phys_footprint 
	// represents our application's total memory footprint rather than the memory
	// that's just in RAM right now (which can vary drastically depending on memory pressure)
	MemoryStats.UsedPhysical = TaskInfo.phys_footprint;
	if(MemoryStats.UsedPhysical > MemoryStats.PeakUsedPhysical)
	{
		MemoryStats.PeakUsedPhysical = MemoryStats.UsedPhysical;
	}
	MemoryStats.UsedVirtual = TaskInfo.virtual_size;
	if(MemoryStats.UsedVirtual > MemoryStats.PeakUsedVirtual)
	{
		MemoryStats.PeakUsedVirtual = MemoryStats.UsedVirtual;
	}
	MemoryStats.MemoryPressureStatus = MemoryPressureStatus;

	return MemoryStats;
}

const FPlatformMemoryConstants& FMacPlatformMemory::GetConstants()
{
	static FPlatformMemoryConstants MemoryConstants;

	if( MemoryConstants.TotalPhysical == 0 )
	{
		// Gather platform memory constants.

		// Get swap file info
		xsw_usage SwapUsage;
		SIZE_T Size = sizeof(SwapUsage);
		sysctlbyname("vm.swapusage", &SwapUsage, &Size, NULL, 0);

		// Get memory.
		int64 AvailablePhysical = 0;
		int Mib[] = {CTL_HW, HW_MEMSIZE};
		size_t Length = sizeof(int64);
		sysctl(Mib, 2, &AvailablePhysical, &Length, NULL, 0);
		
		MemoryConstants.TotalPhysical = AvailablePhysical;
		MemoryConstants.TotalVirtual = AvailablePhysical + SwapUsage.xsu_total;
		MemoryConstants.PageSize = (uint32)vm_page_size;
		MemoryConstants.OsAllocationGranularity = (uint32)vm_page_size;
		MemoryConstants.BinnedPageSize = FMath::Max((SIZE_T)65536, (SIZE_T)vm_page_size);

		MemoryConstants.TotalPhysicalGB = (MemoryConstants.TotalPhysical + 1024 * 1024 * 1024 - 1) / 1024 / 1024 / 1024;
		MemoryConstants.AddressLimit = FPlatformMath::RoundUpToPowerOfTwo64(MemoryConstants.TotalPhysical);
	}

	return MemoryConstants;	
}

FPlatformMemory::FSharedMemoryRegion* FMacPlatformMemory::MapNamedSharedMemoryRegion(const FString& InName, bool bCreate, uint32 AccessMode, SIZE_T Size)
{
	// The maximum is PSHMNAMLEN but a '/' is added below
	constexpr uint32 SHM_MAX_FILENAME = PSHMNAMLEN - 1;
	ensureMsgf(InName.Len() <= SHM_MAX_FILENAME, TEXT("Trying to create a SharedMemoryRegion with a name size > %d will likely failed on Mac (Name=\"%s\", Size=%d)"), SHM_MAX_FILENAME, *InName, InName.Len());
	
	// expecting platform-independent name, so convert it to match platform requirements
	FString Name("/");
	Name += InName;
	FTCHARToUTF8 NameUTF8(*Name);

	// correct size to match platform constraints
	FPlatformMemoryConstants MemConstants = FPlatformMemory::GetConstants();
	check(MemConstants.PageSize > 0);	// also relying on it being power of two, which should be true in foreseeable future
	if (Size & (MemConstants.PageSize - 1))
	{
		Size = Size & ~(MemConstants.PageSize - 1);
		Size += MemConstants.PageSize;
	}

	int ShmOpenFlags = bCreate ? O_CREAT : 0;
	// note that you cannot combine O_RDONLY and O_WRONLY to get O_RDWR
	check(AccessMode != 0);
	if (AccessMode == FPlatformMemory::ESharedMemoryAccess::Read)
	{
		ShmOpenFlags |= O_RDONLY;
	}
	else if (AccessMode == FPlatformMemory::ESharedMemoryAccess::Write)
	{
		ShmOpenFlags |= O_WRONLY;
	}
	else if (AccessMode == (FPlatformMemory::ESharedMemoryAccess::Write | FPlatformMemory::ESharedMemoryAccess::Read))
	{
		ShmOpenFlags |= O_RDWR;
	}

	int ShmOpenMode = (S_IRUSR | S_IWUSR) | (S_IRGRP | S_IWGRP) | (S_IROTH | S_IWOTH );	// 0666

	// open the object
	int SharedMemoryFd = shm_open(NameUTF8.Get(), ShmOpenFlags, ShmOpenMode);
	if (SharedMemoryFd == -1)
	{
		int ErrNo = errno;
		UE_LOGF(LogHAL, Warning, "shm_open(name='%ls', flags=0x%x, mode=0x%x) failed with errno = %d (%ls)", *Name, ShmOpenFlags, ShmOpenMode, ErrNo, 
			StringCast< TCHAR >(strerror(ErrNo)).Get());
		return NULL;
	}

	// truncate if creating (note that we may still don't have rights to do so)
	if (bCreate)
	{
		int Res = ftruncate(SharedMemoryFd, Size);
		if (Res != 0)
		{
			int ErrNo = errno;
			UE_LOGF(LogHAL, Warning, "ftruncate(fd=%d, size=%zu) failed with errno = %d (%ls)", SharedMemoryFd, Size, ErrNo,
				StringCast< TCHAR >(strerror(ErrNo)).Get());
			shm_unlink(NameUTF8.Get());
			return NULL;
		}
	}

	// map
	int MmapProtFlags = 0;
	if (AccessMode & FPlatformMemory::ESharedMemoryAccess::Read)
	{
		MmapProtFlags |= PROT_READ;
	}

	if (AccessMode & FPlatformMemory::ESharedMemoryAccess::Write)
	{
		MmapProtFlags |= PROT_WRITE;
	}

	void *Ptr = mmap(NULL, Size, MmapProtFlags, MAP_SHARED, SharedMemoryFd, 0);
	if (Ptr == MAP_FAILED)
	{
		int ErrNo = errno;
		UE_LOGF(LogHAL, Warning, "mmap(addr=NULL, length=%zu, prot=0x%x, flags=MAP_SHARED, fd=%d, 0) failed with errno = %d (%ls)", Size, MmapProtFlags, SharedMemoryFd, ErrNo,
			StringCast< TCHAR >(strerror(ErrNo)).Get());

		if (bCreate)
		{
			shm_unlink(NameUTF8.Get());
		}
		return NULL;
	}

	return new FMacSharedMemoryRegion(Name, AccessMode, Ptr, Size, SharedMemoryFd, bCreate);
}

bool FMacPlatformMemory::UnmapNamedSharedMemoryRegion(FSharedMemoryRegion * MemoryRegion)
{
	bool bAllSucceeded = true;

	if (MemoryRegion)
	{
		FMacSharedMemoryRegion * MacRegion = static_cast< FMacSharedMemoryRegion* >( MemoryRegion );

		if (munmap(MacRegion->GetAddress(), MacRegion->GetSize()) == -1) 
		{
			bAllSucceeded = false;

			int ErrNo = errno;
			UE_LOGF(LogHAL, Warning, "munmap(addr=%p, len=%zu) failed with errno = %d (%ls)", MacRegion->GetAddress(), MacRegion->GetSize(), ErrNo,
				StringCast< TCHAR >(strerror(ErrNo)).Get());
		}

		if (close(MacRegion->GetFileDescriptor()) == -1)
		{
			bAllSucceeded = false;

			int ErrNo = errno;
			UE_LOGF(LogHAL, Warning, "close(fd=%d) failed with errno = %d (%ls)", MacRegion->GetFileDescriptor(), ErrNo, 
				StringCast< TCHAR >(strerror(ErrNo)).Get());
		}

		if (MacRegion->NeedsToUnlinkRegion())
		{
			FTCHARToUTF8 NameUTF8(MacRegion->GetName());
			if (shm_unlink(NameUTF8.Get()) == -1)
			{
				bAllSucceeded = false;

				int ErrNo = errno;
				UE_LOGF(LogHAL, Warning, "shm_unlink(name='%ls') failed with errno = %d (%ls)", MacRegion->GetName(), ErrNo, 
					StringCast< TCHAR >(strerror(ErrNo)).Get());
			}
		}

		// delete the region
		delete MacRegion;
	}

	return bAllSucceeded;
}
