// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MacPlatformMisc.mm: Mac implementations of misc functions
=============================================================================*/

#include "Mac/MacPlatformMisc.h"
#include "Misc/App.h"
#include "HAL/ExceptionHandling.h"
#include "Misc/SecureHash.h"
#include "Misc/VarargsHelper.h"
#include "Mac/CocoaThread.h"
#include "Misc/EngineVersion.h"
#include "Mac/MacMallocZone.h"
#include "Apple/ApplePlatformSymbolication.h"
#include "Mac/MacGPUDescriptor.h"
#include "Mac/MacPlatformCrashContext.h"
#include "Mac/MacPlatformMiscEx.h"
#include "GenericPlatform/GenericPlatformDriver.h"
#include "HAL/ThreadHeartBeat.h"
#include "HAL/PlatformOutputDevices.h"
#include "HAL/IConsoleManager.h"
#include "HAL/FileManager.h"
#include "HAL/ThreadManager.h"
#include "Misc/OutputDeviceError.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Misc/StringBuilder.h"
#include "Misc/FeedbackContext.h"
#include "Misc/CoreDelegates.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Culture.h"
#include "Modules/ModuleManager.h"
#include "GenericPlatform/GenericPlatformChunkInstall.h"
#include "BuildSettings.h"
#include "PLCrashReporter.h"
#include "Apple/PreAppleSystemHeaders.h"
#include <dlfcn.h>
#include <AppKit/AppKit.h>
#include <IOKit/IOKitLib.h>
#include <IOKit/kext/KextManager.h>
#include <IOKit/network/IOEthernetInterface.h>
#include <IOKit/network/IONetworkInterface.h>
#include <IOKit/network/IOEthernetController.h>
#include <IOKit/ps/IOPowerSources.h>
#include <IOKit/ps/IOPSKeys.h>
#include <IOKit/pwr_mgt/IOPMLib.h>
#include <SystemConfiguration/SystemConfiguration.h>
#include <mach-o/dyld.h>
#include <libproc.h>
#include <notify.h>
#include <uuid/uuid.h>
#include <spawn.h>

#include "Apple/ScopeAutoreleasePool.h"
#include "Apple/AppleStringUtils.h"
#include "Apple/PostAppleSystemHeaders.h"

extern CORE_API bool GIsGPUCrashed;
/*------------------------------------------------------------------------------
 Settings defines.
 ------------------------------------------------------------------------------*/

#if WITH_EDITOR
#define MAC_GRAPHICS_SETTINGS TEXT("/Script/MacGraphicsSwitching.MacGraphicsSwitchingSettings")
#define MAC_GRAPHICS_INI GEditorSettingsIni
#else
#define MAC_GRAPHICS_SETTINGS TEXT("/Script/MacTargetPlatform.MacTargetSettings")
#define MAC_GRAPHICS_INI GEngineIni
#endif

#if MAC_OS_X_VERSION_MAX_ALLOWED < 120000
	// use old constant on old macos, since kIOMainPortDefault was introduced in macOS 12
	#define kIOMainPortDefault kIOMasterPortDefault
#endif

/*------------------------------------------------------------------------------
 Console variables.
 ------------------------------------------------------------------------------*/

/** The selected explicit renderer ID. */
static int32 GMacExplicitRendererID = -1;
static FAutoConsoleVariableRef CVarMacExplicitRendererID(
	TEXT("Mac.ExplicitRendererID"),
	GMacExplicitRendererID,
	TEXT("Forces the Mac RHI to use the specified rendering device which is a 0-based index into the list of GPUs provided by FMacPlatformMisc::GetGPUDescriptors or -1 to disable & use the default device. (Default: -1, off)"),
	ECVF_RenderThreadSafe|ECVF_ReadOnly
	);
static TAutoConsoleVariable<int32> CVarMacPlatformDumpAllThreadsOnHang(
	TEXT("Mac.DumpAllThreadsOnHang"),
	1,
	TEXT("If > 0, then when reporting a hang generate a backtrace for all threads."));

/*------------------------------------------------------------------------------
 Platform property discovery.
 ------------------------------------------------------------------------------*/

#define PLATFORM_MAC_IOSERVICE_MATCHING_NAME_ARM64	"AppleARMIODevice"
#define PLATFORM_MAC_IOSERVICE_MATCHING_NAME_X86	"IOPCIDevice"
#define PLATFORM_MAC_CLASS_CODE_NAME_ARM64			"device_type"
#define PLATFORM_MAC_CLASS_CODE_NAME_X86			"class-code"

#define PLATFORM_MAC_MAKE_FOURCC(ch0, ch1, ch2, ch3)							\
   ((uint32_t)(uint8_t)(ch0)		| ((uint32_t)(uint8_t)(ch1) << 8	)	|	\
   ((uint32_t)(uint8_t)(ch2) << 16) | ((uint32_t)(uint8_t)(ch3) << 24	))

#if PLATFORM_MAC_ARM64

inline bool IsRunningOnAppleSilicon()
{
	return true;
}

static const char* GetIOServiceMatchingName()
{
	return PLATFORM_MAC_IOSERVICE_MATCHING_NAME_ARM64;
}

static const char* GetClassCode()
{
	return PLATFORM_MAC_CLASS_CODE_NAME_ARM64;
}

#elif PLATFORM_MAC_X86

#define MAC_PROCESS_TYPE_NATIVE		 0
#define MAC_PROCESS_TYPE_TRANSLATED	 1
#define MAC_PROCESS_TYPE_UNKNOWN	-1

static int32 GetProcessTranslationType()
{
	static int32 MacProcessType = MAC_PROCESS_TYPE_UNKNOWN;

	if (MacProcessType == MAC_PROCESS_TYPE_UNKNOWN)
	{
		int32 Value = 0;
		size_t ValueSize = sizeof(Value);
		if (sysctlbyname("sysctl.proc_translated", &Value, &ValueSize, NULL, 0) == -1)
		{
			if (errno == ENOENT)
			{
				Value = MAC_PROCESS_TYPE_NATIVE;
			}
		}
		MacProcessType = Value;
	}

	return MacProcessType;
}

inline bool IsRunningOnAppleSilicon()
{
	return MAC_PROCESS_TYPE_TRANSLATED == GetProcessTranslationType();
}

static const char* GetIOServiceMatchingName()
{
	return IsRunningOnAppleSilicon() ? PLATFORM_MAC_IOSERVICE_MATCHING_NAME_ARM64 : PLATFORM_MAC_IOSERVICE_MATCHING_NAME_X86;
}

static const char* GetClassCode()
{
	return IsRunningOnAppleSilicon() ? PLATFORM_MAC_CLASS_CODE_NAME_ARM64 : PLATFORM_MAC_CLASS_CODE_NAME_X86;
}

#else
	#error "Undefined Mac platform"

#endif // PLATFORM_MAC_XXXXX

/*------------------------------------------------------------------------------
 FMacApplicationInfo - class to contain all state for crash reporting that is unsafe to acquire in a signal.
 ------------------------------------------------------------------------------*/

 CORE_API FMacMallocCrashHandler* GCrashMalloc = nullptr;

// e.g. 13.2.1
static void MacPlatformGetOSProductVersion(FString &OutOSVersion)
{
	static FString OSVersion;
	if (OSVersion.IsEmpty())
	{
		SIZE_T OSProductVersionStringSize = 0;
		if (0 == (sysctlbyname("kern.osproductversion", NULL, &OSProductVersionStringSize, NULL, 0)))
		{
			ANSICHAR *OSProductVersionCString = new ANSICHAR[OSProductVersionStringSize];
			if (0 == (sysctlbyname("kern.osproductversion", OSProductVersionCString, &OSProductVersionStringSize, NULL, 0)))
			{
				OSVersion = ANSI_TO_TCHAR(OSProductVersionCString);
			}
			delete [] OSProductVersionCString;
		}
	}
	OutOSVersion = OSVersion;
}

// e.g. 22D68
static void MacPlatformGetOSVersion(FString &OutOSBuild)
{
	static FString OSBuild;
	if (OSBuild.IsEmpty())
	{
		SIZE_T OSVersionStringSize = 0;
		if (0 == (sysctlbyname("kern.osversion", NULL, &OSVersionStringSize, NULL, 0)))
		{
			ANSICHAR *OSVersionCString = new ANSICHAR[OSVersionStringSize];
			if (0 == (sysctlbyname("kern.osversion", OSVersionCString, &OSVersionStringSize, NULL, 0)))
			{
				OSBuild = ANSI_TO_TCHAR(OSVersionCString);
			}
			delete [] OSVersionCString;
		}
	}
	OutOSBuild = OSBuild;
}

/**
 * Information that cannot be obtained during a signal-handler is initialised here.
 * This ensures that we only call safe functions within the crash reporting handler.
 */
struct FMacApplicationInfo
{
	void Init()
	{
		SCOPED_AUTORELEASE_POOL;
		
		// Prevent clang/ld from dead-code-eliminating the nothrow_t variants of global new.
		// This ensures that all OS calls to operator new go through our allocators and delete cleanly.
		{
			std::nothrow_t t;
			char* d = (char*)(operator new(8, t));
			delete d;
			
			d = (char*)operator new[]( 8, t );
			delete [] d;
		}
		
		AppName = FApp::GetProjectName();
		FCStringAnsi::Strncpy(AppNameUTF8, TCHAR_TO_UTF8(*AppName), PATH_MAX + 1);
		
		ExecutableName = FPlatformProcess::ExecutableName();
		
		AppPath = FString([[NSBundle mainBundle] executablePath]);

		AppBundleID = FString([[NSBundle mainBundle] bundleIdentifier]);
		
		bIsUnattended = FApp::IsUnattended();

		bIsSandboxed = FPlatformProcess::IsSandboxedApplication();
		
		NumCores = FPlatformMisc::NumberOfCores();
		
		LCID = FString::Printf(TEXT("%d"), FInternationalization::Get().GetCurrentCulture()->GetLCID());
		
		PrimaryGPU = FPlatformMisc::GetPrimaryGPUBrand();
		
		RunUUID = RunGUID();
		
		MacPlatformGetOSProductVersion(OSVersion);
		FCStringAnsi::Strncpy(OSVersionUTF8, TCHAR_TO_UTF8(*OSVersion), PATH_MAX + 1);
		
		MacPlatformGetOSVersion(OSBuild);

		OSXVersion = FMacPlatformMiscEx::GetNSOperatingSystemVersion();

		XcodeVersion.majorVersion = XcodeVersion.minorVersion = XcodeVersion.patchVersion = 0;

		FPlatformProcess::ExecProcess(TEXT("/usr/bin/xcode-select"), TEXT("--print-path"), nullptr, &XcodePath, nullptr);
		if (XcodePath.Len() > 0)
		{
			XcodePath.RemoveAt(XcodePath.Len() - 1); // Remove \n at the end of the string
			if (IFileManager::Get().DirectoryExists(*XcodePath))
			{
				FString XcodeAppPath = XcodePath.Left(XcodePath.Find(TEXT(".app/")) + 4);
				NSBundle* XcodeBundle = [NSBundle bundleWithPath:XcodeAppPath.GetNSString()];
				if (XcodeBundle)
				{
					NSString* XcodeVersionString = (NSString*)[XcodeBundle objectForInfoDictionaryKey:@"CFBundleShortVersionString"];
					if (XcodeVersionString)
					{
						NSArray<NSString*>* VersionComponents = [XcodeVersionString componentsSeparatedByString:@"."];
						XcodeVersion.majorVersion = [[VersionComponents objectAtIndex:0] integerValue];
						XcodeVersion.minorVersion = VersionComponents.count > 1 ? [[VersionComponents objectAtIndex:1] integerValue] : 0;
						XcodeVersion.patchVersion = VersionComponents.count > 2 ? [[VersionComponents objectAtIndex:2] integerValue] : 0;
					}
				}
			}

			if (XcodeVersion.majorVersion == 0)
			{
				XcodePath.Empty();
			}
		}

		char TempSysCtlBuffer[PATH_MAX] = {};
		size_t TempSysCtlBufferSize = PATH_MAX;
		
		pid_t ParentPID = getppid();
		proc_pidpath(ParentPID, TempSysCtlBuffer, PATH_MAX);
		ParentProcess = TempSysCtlBuffer;
		
		MachineUUID = TEXT("00000000-0000-0000-0000-000000000000");
		io_service_t PlatformExpert = IOServiceGetMatchingService(kIOMainPortDefault,IOServiceMatching("IOPlatformExpertDevice"));
		if(PlatformExpert)
		{
			CFTypeRef SerialNumberAsCFString = IORegistryEntryCreateCFProperty(PlatformExpert,CFSTR(kIOPlatformUUIDKey),kCFAllocatorDefault, 0);
			if(SerialNumberAsCFString)
			{
				MachineUUID = FString((NSString*)SerialNumberAsCFString);
				CFRelease(SerialNumberAsCFString);
			}
			IOObjectRelease(PlatformExpert);
		}
		
		sysctlbyname("kern.osrelease", TempSysCtlBuffer, &TempSysCtlBufferSize, NULL, 0);
		BiosRelease = TempSysCtlBuffer;
		uint32 KernelRevision = 0;
		TempSysCtlBufferSize = 4;
		sysctlbyname("kern.osrevision", &KernelRevision, &TempSysCtlBufferSize, NULL, 0);
		BiosRevision = FString::Printf(TEXT("%d"), KernelRevision);
		TempSysCtlBufferSize = PATH_MAX;
		sysctlbyname("kern.uuid", TempSysCtlBuffer, &TempSysCtlBufferSize, NULL, 0);
		BiosUUID = TempSysCtlBuffer;
		TempSysCtlBufferSize = PATH_MAX;
		sysctlbyname("hw.model", TempSysCtlBuffer, &TempSysCtlBufferSize, NULL, 0);
		MachineModel = TempSysCtlBuffer;
		TempSysCtlBufferSize = PATH_MAX+1;
		sysctlbyname("machdep.cpu.brand_string", MachineCPUString, &TempSysCtlBufferSize, NULL, 0);
		
		gethostname(MachineName, UE_ARRAY_COUNT(MachineName));
		
		FString CrashVideoPath = FPaths::ProjectLogDir() + TEXT("CrashVideo.avi");

		// The engine mode may be incorrect at this point, as GIsEditor is uninitialized yet. We'll update BranchBaseDir in PostInitUpdate(),
		// but we initialize it here anyway in case the engine crashes before PostInitUpdate() is called.
		BranchBaseDir = FString::Printf( TEXT( "%s!%s!%s!%d" ), *FApp::GetBranchName(), FPlatformProcess::BaseDir(), FPlatformMisc::GetEngineMode(), FEngineVersion::Current().GetChangelist() );
		
		// Get the paths that the files will actually have been saved to
		FString LogDirectory = FPaths::ProjectLogDir();
		TCHAR CommandlineLogFile[MAX_SPRINTF]=TEXT("");
		
		// Use the log file specified on the commandline if there is one
		CommandLine = FCommandLine::Get();
		FString LogPath = FGenericPlatformOutputDevices::GetAbsoluteLogFilename();
		FCStringAnsi::Strncpy(AppLogPath, TCHAR_TO_UTF8(*LogPath), PATH_MAX + 1);

		FString UserCrashVideoPath = IFileManager::Get().ConvertToAbsolutePathForExternalAppForWrite(*CrashVideoPath);
		FCStringAnsi::Strncpy(CrashReportVideo, TCHAR_TO_UTF8(*UserCrashVideoPath), PATH_MAX + 1);
		
		// Cache & create the crash report folder.
		FString ReportPath = FPaths::ConvertRelativePathToFull(FString::Printf(TEXT("%s"), *(FPaths::GameAgnosticSavedDir() / TEXT("Crashes"))));
		FCStringAnsi::Strncpy(CrashReportPath, TCHAR_TO_UTF8(*ReportPath), PATH_MAX + 1);
#if WITH_EDITOR
		FString ReportClient = FPaths::ConvertRelativePathToFull(FPlatformProcess::GenerateApplicationPath(TEXT("CrashReportClientEditor"), EBuildConfiguration::Development));
#else
		FString ReportClient = FPaths::ConvertRelativePathToFull(FPlatformProcess::GenerateApplicationPath(TEXT("CrashReportClient"), EBuildConfiguration::Development));
#endif
		FCStringAnsi::Strncpy(CrashReportClient, TCHAR_TO_UTF8(*ReportClient), PATH_MAX + 1);
		IFileManager::Get().MakeDirectory(*ReportPath, true);
		
		// Notification handler to check we are running from a battery - this only applies to MacBook's.
		notify_handler_t PowerSourceNotifyHandler = ^(int32 Token){
			RunningOnBattery = false;
			CFTypeRef PowerSourcesInfo = IOPSCopyPowerSourcesInfo();
			if (PowerSourcesInfo)
			{
				CFArrayRef PowerSourcesArray = IOPSCopyPowerSourcesList(PowerSourcesInfo);
				for (CFIndex Index = 0; Index < CFArrayGetCount(PowerSourcesArray); Index++)
				{
					CFTypeRef PowerSource = CFArrayGetValueAtIndex(PowerSourcesArray, Index);
					NSDictionary* Description = (NSDictionary*)IOPSGetPowerSourceDescription(PowerSourcesInfo, PowerSource);
					if ([(NSString*)[Description objectForKey: @kIOPSPowerSourceStateKey] isEqualToString: @kIOPSBatteryPowerValue])
					{
						RunningOnBattery = true;
						break;
					}
				}
				CFRelease(PowerSourcesArray);
				CFRelease(PowerSourcesInfo);
			}
		};
		
		// Call now to fetch the status
		PowerSourceNotifyHandler(0);
		
		uint32 Status = notify_register_dispatch(kIOPSNotifyPowerSource, &PowerSourceNotification, dispatch_get_main_queue(), PowerSourceNotifyHandler);
		check(Status == NOTIFY_STATUS_OK);
		
		NumCores = FPlatformMisc::NumberOfCores();
		
		NSString* PLCrashReportFile = [TemporaryCrashReportFolder().GetNSString() stringByAppendingPathComponent:TemporaryCrashReportName().GetNSString()];
		[PLCrashReportFile getCString:PLCrashReportPath maxLength:PATH_MAX encoding:NSUTF8StringEncoding];
		
		SystemLogSize = 0;
		KernelErrorDir = nullptr;
		if (!bIsSandboxed)
		{
			SystemLogSize = IFileManager::Get().FileSize(TEXT("/var/log/system.log"));
			
			KernelErrorDir = opendir("/Library/Logs/DiagnosticReports");
		}
		
		if (!FPlatformMisc::IsDebuggerPresent() && FParse::Param(FCommandLine::Get(), TEXT("RedirectNSLog")))
		{
			fflush(stderr);
			StdErrPipe = [NSPipe new];
			int StdErr = dup2([StdErrPipe fileHandleForWriting].fileDescriptor, STDERR_FILENO);
			if(StdErr > 0)
			{
				@try
				{
					NSFileHandle* StdErrFile = [StdErrPipe fileHandleForReading];
					if(StdErrFile)
					{
						StdErrFile.readabilityHandler = ^(NSFileHandle* Handle){
							NSData* FileData = Handle.availableData;
							if (FileData.length > 0)
							{
								NSString* NewString = (NSString*)[[[NSString alloc] initWithData:FileData encoding:NSUTF8StringEncoding] autorelease];
								UE_LOGF(LogMac, Error, "NSLog: %ls", *FString(NewString));
							}
						};
					}
				}
				@catch (NSException* Exc)
				{
					UE_LOGF(LogMac, Warning, "Exception redirecting stderr to capture NSLog messages: %ls", *FString([Exc description]));
					[StdErrPipe release];
					StdErrPipe = nil;
				}
			}
			else
			{
				UE_LOGF(LogMac, Warning, "Failed to redirect stderr in order to capture NSLog messages.");
				[StdErrPipe release];
				StdErrPipe = nil;
			}
		}
	}
	
	~FMacApplicationInfo()
	{
		if (UE::Private::GMalloc != GCrashMalloc)
		{
			delete GCrashMalloc;
		}
		if(CrashReporter)
		{
			CrashReporter = nil;
			[CrashReporter release];
		}
		if(PowerSourceNotification)
		{
			notify_cancel(PowerSourceNotification);
			PowerSourceNotification = 0;
		}
		if (KernelErrorDir)
		{
			closedir(KernelErrorDir);
			KernelErrorDir = nullptr;
		}
	}
	
	static FGuid RunGUID()
	{
		static FGuid Guid;
		if(!Guid.IsValid())
		{
			FPlatformMisc::CreateGuid(Guid);
		}
		return Guid;
	}
	
	static FString TemporaryCrashReportFolder()
	{
		static FString PLCrashReportFolder;
		if(PLCrashReportFolder.IsEmpty())
		{
			SCOPED_AUTORELEASE_POOL;
			
			NSArray* Paths = NSSearchPathForDirectoriesInDomains(NSCachesDirectory, NSUserDomainMask, YES);
			NSString* CacheDir = [Paths objectAtIndex: 0];
			
			NSString* BundleID = [[NSBundle mainBundle] bundleIdentifier];
			if(!BundleID)
			{
				BundleID = [[NSProcessInfo processInfo] processName];
			}
			check(BundleID);
			
			NSString* PLCrashReportFolderPath = [CacheDir stringByAppendingPathComponent: BundleID];
			PLCrashReportFolder = FString(PLCrashReportFolderPath);
		}
		return PLCrashReportFolder;
	}
	
	static FString TemporaryCrashReportName()
	{
		static FString PLCrashReportFileName(RunGUID().ToString() + TEXT(".plcrash"));
		return PLCrashReportFileName;
	}
	
	bool bIsUnattended;
	bool bIsSandboxed;
	bool RunningOnBattery;
	int32 PowerSourceNotification;
	int32 NumCores;
	int64 SystemLogSize;
	char AppNameUTF8[PATH_MAX+1];
	char AppLogPath[PATH_MAX+1];
	char CrashReportPath[PATH_MAX+1];
	char PLCrashReportPath[PATH_MAX+1];
	char CrashReportClient[PATH_MAX+1];
	char CrashReportVideo[PATH_MAX+1];
	char OSVersionUTF8[PATH_MAX+1];
	char MachineName[PATH_MAX+1];
	char MachineCPUString[PATH_MAX+1];
	FString AppPath;
	FString AppName;
	FString AppBundleID;
	FString OSVersion;
	FString OSBuild;
	FString MachineUUID;
	FString MachineModel;
	FString BiosRelease;
	FString BiosRevision;
	FString BiosUUID;
	FString ParentProcess;
	FString LCID;
	FString CommandLine;
	FString BranchBaseDir;
	FString PrimaryGPU;
	FString ExecutableName;
	NSOperatingSystemVersion OSXVersion;
	FGuid RunUUID;
	FString XcodePath;
	NSOperatingSystemVersion XcodeVersion;
	NSPipe* StdErrPipe;
	DIR* KernelErrorDir;
	static PLCrashReporter* CrashReporter;
};
static FMacApplicationInfo GMacAppInfo;
PLCrashReporter* FMacApplicationInfo::CrashReporter = nullptr;

void FMacPlatformMisc::PlatformPreInit()
{
	FGenericPlatformMisc::PlatformPreInit();
	
	GMacAppInfo.Init();
	
	// No SIGPIPE crashes please - they are a pain to debug!
	signal(SIGPIPE, SIG_IGN);

	// Disable ApplePlatformThreadStackWalk when the debugger is attached
	if (FPlatformMisc::IsDebuggerPresent() && !GAlwaysReportCrash)
	{
		IConsoleVariable* CVarApplePlatformThreadStackWalkEnable = IConsoleManager::Get().FindConsoleVariable(TEXT("ApplePlatformThreadStackWalk.Enable"));
		if (CVarApplePlatformThreadStackWalkEnable)
		{
			CVarApplePlatformThreadStackWalkEnable->Set(0);
		}
	}

	// Increase the maximum number of simultaneously open files
	uint32 MaxFilesPerProc = OPEN_MAX;
	size_t UInt32Size = sizeof(uint32);
	sysctlbyname("kern.maxfilesperproc", &MaxFilesPerProc, &UInt32Size, NULL, 0);

	struct rlimit Limit = {MaxFilesPerProc, RLIM_INFINITY};
	int32 Result = getrlimit(RLIMIT_NOFILE, &Limit);
	if (Result == 0)
	{
		if (Limit.rlim_max == RLIM_INFINITY)
		{
			Limit.rlim_cur = MaxFilesPerProc;
		}
		else
		{
			Limit.rlim_cur = FMath::Min(Limit.rlim_max, (rlim_t)MaxFilesPerProc);
		}
	}
	if (Limit.rlim_cur < OPEN_MAX)
	{
		UE_LOGF(LogInit, Warning, "Open files limit too small: %llu, should be at least OPEN_MAX (%u). rlim_max is %llu, kern.maxfilesperproc is %u. UE may be unstable.", Limit.rlim_cur, OPEN_MAX, Limit.rlim_max, MaxFilesPerProc);
	}
	Result = setrlimit(RLIMIT_NOFILE, &Limit);
	if (Result != 0)
	{
		UE_LOGF(LogInit, Warning, "Failed to change open file limit, UE may be unstable.");
	}

	FApplePlatformSymbolication::EnableCoreSymbolication(!FPlatformProcess::IsSandboxedApplication() && IS_PROGRAM);
}

void FMacPlatformMisc::PlatformInit()
{
	UE_LOGF(LogInit, Log, "macOS %ls (%ls)", *GMacAppInfo.OSVersion, *GMacAppInfo.OSBuild);
	UE_LOGF(LogInit, Log, "Model: %ls", *GMacAppInfo.MachineModel);
	UE_LOGF(LogInit, Log, "CPU: %ls", UTF8_TO_TCHAR(GMacAppInfo.MachineCPUString));

	// At the point, the log system is booted and the log file is likely created and the actual pathname used set. (Until the file is opened for writing, the name is subject to change)
	FString LogPath = FGenericPlatformOutputDevices::GetAbsoluteLogFilename();
	FCStringAnsi::Strncpy(GMacAppInfo.AppLogPath, TCHAR_TO_UTF8(*LogPath), PATH_MAX + 1);

	const FPlatformMemoryConstants& MemoryConstants = FPlatformMemory::GetConstants();
	UE_LOGF(LogInit, Log, "CPU Page size=%zu, Cores=%i, HT=%i", MemoryConstants.PageSize, FPlatformMisc::NumberOfCores(), FPlatformMisc::NumberOfCoresIncludingHyperthreads() );

	// Identity.
	UE_LOGF(LogInit, Log, "Computer: %ls", FPlatformProcess::ComputerName() );
	UE_LOGF(LogInit, Log, "User: %ls", FPlatformProcess::UserName() );

	// Timer resolution.
	UE_LOGF(LogInit, Log, "High frequency timer resolution =%f MHz", 0.000001 / FPlatformTime::GetSecondsPerCycle() );
	
	UE_LOGF(LogInit, Log, "Power Source: %ls", GMacAppInfo.RunningOnBattery ? TEXT(kIOPSBatteryPowerValue) : TEXT(kIOPSACPowerValue) );

#if WITH_EDITOR
	if (GMacAppInfo.XcodePath.Len())
	{
		UE_LOGF(LogInit, Log, "Xcode developer folder path: %ls, version %lld.%lld.%lld", *GMacAppInfo.XcodePath, GMacAppInfo.XcodeVersion.majorVersion, GMacAppInfo.XcodeVersion.minorVersion, GMacAppInfo.XcodeVersion.patchVersion);
	}
	else
	{
		UE_LOGF(LogInit, Log, "No Xcode installed");
	}
#endif
	
	FApplePlatformMisc::PlatformInit();
}

void FMacPlatformMisc::PostInitMacAppInfoUpdate()
{
	GMacAppInfo.BranchBaseDir = FString::Printf(TEXT("%s!%s!%s!%d"), *FApp::GetBranchName(), FPlatformProcess::BaseDir(), FPlatformMisc::GetEngineMode(), FEngineVersion::Current().GetChangelist());
}

void FMacPlatformMisc::PlatformTearDown()
{
	FApplePlatformSymbolication::EnableCoreSymbolication(false);
	
	if (GMacAppInfo.StdErrPipe)
	{
		NSFileHandle* StdErrFile = [GMacAppInfo.StdErrPipe fileHandleForReading];
		if (StdErrFile)
		{
			StdErrFile.readabilityHandler = nil;
		}
		
		[GMacAppInfo.StdErrPipe release];
	}
	FPlatformProcess::CeaseBeingFirstInstance();
	FApplePlatformMisc::PlatformTearDown();
}

TArray<uint8> FMacPlatformMisc::GetMacAddress()
{
	TArray<uint8> Result;

	io_iterator_t InterfaceIterator;
	{
		CFMutableDictionaryRef MatchingDict = IOServiceMatching(kIOEthernetInterfaceClass);

		if (!MatchingDict)
		{
			UE_LOGF(LogMac, Warning, "GetMacAddress failed - no Ethernet interfaces");
			return Result;
		}

		CFMutableDictionaryRef PropertyMatchDict =
			CFDictionaryCreateMutable(kCFAllocatorDefault, 0,
			&kCFTypeDictionaryKeyCallBacks, &kCFTypeDictionaryValueCallBacks);

		if (!PropertyMatchDict)
		{
			UE_LOGF(LogMac, Warning, "GetMacAddress failed - can't create CoreFoundation mutable dictionary!");
			return Result;
		}

		CFDictionarySetValue(PropertyMatchDict, CFSTR(kIOPrimaryInterface), kCFBooleanTrue);
		CFDictionarySetValue(MatchingDict, CFSTR(kIOPropertyMatchKey), PropertyMatchDict);
		CFRelease(PropertyMatchDict);

		if (IOServiceGetMatchingServices(kIOMainPortDefault, MatchingDict, &InterfaceIterator) != KERN_SUCCESS)
		{
			UE_LOGF(LogMac, Warning, "GetMacAddress failed - error getting matching services");
			return Result;
		}
	}

	io_object_t InterfaceService;
	while ( (InterfaceService = IOIteratorNext(InterfaceIterator)) != 0)
	{
		io_object_t ControllerService;
		if (IORegistryEntryGetParentEntry(InterfaceService, kIOServicePlane, &ControllerService) == KERN_SUCCESS)
		{
			CFTypeRef MACAddressAsCFData = IORegistryEntryCreateCFProperty(
				ControllerService, CFSTR(kIOMACAddress), kCFAllocatorDefault, 0);
			if (MACAddressAsCFData)
			{
				Result.AddZeroed(kIOEthernetAddressSize);
				CFDataGetBytes((CFDataRef)MACAddressAsCFData, CFRangeMake(0, kIOEthernetAddressSize), Result.GetData());
				CFRelease(MACAddressAsCFData);
				break;
			}
			IOObjectRelease(ControllerService);
		}
		IOObjectRelease(InterfaceService);
	}
	IOObjectRelease(InterfaceIterator);

	return Result;
}

void FMacPlatformMisc::RequestExit( bool Force, const TCHAR* CallSite)
{
	UE_LOGF(LogMac, Log,  "FPlatformMisc::RequestExit(%i, %ls)", Force,
		CallSite ? CallSite : TEXT("<NoCallSiteInfo>"));

	if (!UE::IsGCLocked())
	{
		FCoreDelegates::GetApplicationWillTerminateDelegate().Broadcast();
	}
	
	notify_cancel(GMacAppInfo.PowerSourceNotification);
	GMacAppInfo.PowerSourceNotification = 0;
	
	if( Force )
	{
		// Make sure the log is flushed.
		if (GLog)
		{
			GLog->Flush();
		}


		// Exit immediately, by request.
		_Exit(GIsCriticalError ? 3 : 0);
	}
	else
	{
		// Tell the platform specific code we want to exit cleanly from the main loop.
		RequestEngineExit(TEXT("Mac RequestExit"));
	}
}

CORE_API TFunction<EAppReturnType::Type(EAppMsgType::Type MsgType, const TCHAR* Text, const TCHAR* Caption)> MessageBoxExtCallback;

EAppReturnType::Type FMacPlatformMisc::MessageBoxExt(EAppMsgType::Type MsgType, const TCHAR* Text, const TCHAR* Caption)
{
	if(MessageBoxExtCallback)
	{
		return MessageBoxExtCallback(MsgType, Text, Caption);
	}
	else
	{
		return FGenericPlatformMisc::MessageBoxExt(MsgType, Text, Caption);
	}
}

static bool HandleFirstInstall()
{
	if (FParse::Param( FCommandLine::Get(), TEXT("firstinstall")))
	{
		GLog->Flush();

		// Flush config to ensure language changes are written to disk.
		GConfig->Flush(false);

		return false; // terminate the game
	}
	return true; // allow the game to continue;
}

bool FMacPlatformMisc::CommandLineCommands()
{
	return HandleFirstInstall();
}

const TCHAR* FMacPlatformMisc::GetNullRHIShaderFormat()
{
	if (FParse::Param(FCommandLine::Get(), TEXT("sm5")))
	{
		return TEXT("SF_METAL_SM5");
	}
	else
	{
		return TEXT("SF_METAL_SM6");
	}
}

int32 FMacPlatformMisc::NumberOfCores()
{	
	static int32 NumberOfCores = -1;
	if (NumberOfCores == -1)
	{
		if (FParse::Param(FCommandLine::Get(), TEXT("usehyperthreading")))
		{
			NumberOfCores = NumberOfCoresIncludingHyperthreads();
		}
		else
		{
			SIZE_T Size = sizeof(int32);
			int Result = sysctlbyname("hw.physicalcpu", &NumberOfCores, &Size, NULL, 0);
		
			if (Result != 0)
			{
				UE_LOGF(LogMac, Error,  "sysctlbyname(hw.physicalcpu...) failed with error %d. Defaulting to one core", Result);
				NumberOfCores = 1;
			}
		}
	}
	return NumberOfCores;
}

int32 FMacPlatformMisc::NumberOfCoresIncludingHyperthreads()
{
	static int32 NumberOfCores = -1;
	if (NumberOfCores == -1)
	{
		SIZE_T Size = sizeof(int32);
		int Result = sysctlbyname("hw.logicalcpu", &NumberOfCores, &Size, NULL, 0);
		
		if (Result != 0)
		{
			UE_LOGF(LogMac, Error,  "sysctlbyname(hw.logicalcpu...) failed with error %d. Defaulting to one core", Result);
			NumberOfCores = 1;
		}
	}
	return NumberOfCores;
}

void FMacPlatformMisc::NormalizePath(FString& InPath)
{
	// only expand if path starts with ~, e.g. ~/ should be expanded, /~ should not
	if (InPath.StartsWith(TEXT("~"), ESearchCase::CaseSensitive))	// case sensitive is quicker, and our substring doesn't care
	{
		InPath = InPath.Replace(TEXT("~"), FPlatformProcess::UserHomeDir(), ESearchCase::CaseSensitive);
	}
}

void FMacPlatformMisc::NormalizePath(FStringBuilderBase& InPath)
{
	// only expand if path starts with ~, e.g. ~/ should be expanded, /~ should not
	if (FStringView(InPath).StartsWith('~'))
	{
		InPath.ReplaceAt(0, 1, FPlatformProcess::UserHomeDir());
	}
}

template<typename T>
FMacGPUDescriptorCommon<T>::~FMacGPUDescriptorCommon()
{
	[GPUName release];
	[GPUMetalBundle release];
	[GPUOpenGLBundle release];
	[GPUBundleID release];
}

template<typename T>
void FMacGPUDescriptorCommon<T>::CopyFrom(FMacGPUDescriptorCommon<T> const& Other)
{
	if (this != &Other)
	{
		[Other.GPUName retain];
		[Other.GPUMetalBundle retain];
		[Other.GPUOpenGLBundle retain];
		[Other.GPUBundleID retain];

		[GPUName release];
		[GPUMetalBundle release];
		[GPUOpenGLBundle release];
		[GPUBundleID release];

		GPUName = Other.GPUName;
		GPUMetalBundle = Other.GPUMetalBundle;
		GPUOpenGLBundle = Other.GPUOpenGLBundle;
		GPUBundleID = Other.GPUBundleID;

		GPUVendorId = Other.GPUVendorId;
		GPUDeviceId = Other.GPUDeviceId;
		GPUMemoryMB = Other.GPUMemoryMB;
		GPUIndex = Other.GPUIndex;
		GPUHeadless = Other.GPUHeadless;

		static_cast<T*>(this)->CopyFromImpl(Other);
	}
}

template<typename T>
FMacGPUDescriptorCommon<T>& FMacGPUDescriptorCommon<T>::operator=(FMacGPUDescriptorCommon<T> const& Other)
{
	CopyFrom(Other);
	return *this;
}

template<typename T>
TMap<FString, float> FMacGPUDescriptorCommon<T>::GetPerformanceStatistics() const
{
	return static_cast<T const*>(this)->GetPerformanceStatisticsImpl();
}

#if PLATFORM_MAC_X86
FMacGPUDescriptor::FMacGPUDescriptor(FMacGPUDescriptor const& Other)
{
	CopyFrom(Other);
}

FMacGPUDescriptor::~FMacGPUDescriptor()
{
	if (PCIDevice)
	{
		IOObjectRelease((io_registry_entry_t)PCIDevice);
	}
}

void FMacGPUDescriptor::CopyFromImpl(FMacGPUDescriptorCommon<FMacGPUDescriptor> const& Other)
{
	FMacGPUDescriptor const& ConcreteOther = static_cast<FMacGPUDescriptor const&>(Other);

	RegistryID = ConcreteOther.RegistryID;

	if (ConcreteOther.PCIDevice)
	{
		IOObjectRetain((io_registry_entry_t)ConcreteOther.PCIDevice);
	}
	if(PCIDevice)
	{
		IOObjectRelease((io_registry_entry_t)PCIDevice);
	}
	PCIDevice = ConcreteOther.PCIDevice;
}

TMap<FString, float> FMacGPUDescriptor::GetPerformanceStatisticsImpl() const
{
	SCOPED_AUTORELEASE_POOL;

	static CFStringRef PerformanceStatisticsRef = CFSTR("PerformanceStatistics");
	TMap<FString, float> Data;

	const CFDictionaryRef PerformanceStats = (const CFDictionaryRef)IORegistryEntrySearchCFProperty(PCIDevice, kIOServicePlane, PerformanceStatisticsRef, kCFAllocatorDefault, kIORegistryIterateRecursively);
	if(PerformanceStats)
	{
		if(CFGetTypeID(PerformanceStats) == CFDictionaryGetTypeID())
		{
			NSDictionary* PerformanceStatistics = (NSDictionary*)PerformanceStats;
			for(NSString* Key in PerformanceStatistics)
			{
				NSNumber* Value = [PerformanceStatistics objectForKey:Key];
				Data.Add(FString(Key), [Value floatValue]);
			}
		}
		CFRelease(PerformanceStats);
	}

	return Data;
}

#elif PLATFORM_MAC_ARM64
FMacGPUDescriptor::FMacGPUDescriptor(FMacGPUDescriptor const& Other)
{
	CopyFrom(Other);
}

FMacGPUDescriptor::~FMacGPUDescriptor()
{
}

void FMacGPUDescriptor::CopyFromImpl(FMacGPUDescriptorCommon<FMacGPUDescriptor> const& Other)
{
	FMacGPUDescriptor const& ConcreteOther = static_cast<FMacGPUDescriptor const&>(Other);

	RegistryID = ConcreteOther.RegistryID;
}

TMap<FString, float> FMacGPUDescriptor::GetPerformanceStatisticsImpl() const
{
	return TMap<FString, float>();
}

#else
	#error "Undefined Mac platform"

#endif // PLATFORM_MAC_XXXXXX

class FMacPlatformGPUManager
{
	FCriticalSection Mutex;
	TArray<FMacGPUDescriptor> CurrentGPUs;
	TArray<FMacGPUDescriptor> UpdatedGPUs;
	TAtomic<bool> bRequiresUpdate;

	void InitializeDescriptorFromDeviceEntryM(FMacGPUDescriptor& Desc, io_registry_entry_t ServiceEntry, CFMutableDictionaryRef ServiceInfo)
	{
		static CFStringRef IOMatchCategoryRef	= CFSTR("IOMatchCategory");
		static CFStringRef IOAcceleratorRef		= CFSTR("IOAccelerator");
		static CFStringRef CFBundleIdentifier	= CFSTR("CFBundleIdentifier");
		static CFStringRef VendorIDRef			= CFSTR("vendor-id");
		static CFStringRef MetalPluginNameRef	= CFSTR("MetalPluginName");
		static CFStringRef GLBundleNameRef		= CFSTR("IOGLBundleName");
		static CFStringRef ModelRef				= CFSTR("model");

		io_iterator_t ChildIterator;
		if(IORegistryEntryGetChildIterator(ServiceEntry, kIOServicePlane, &ChildIterator) == kIOReturnSuccess)
		{
			io_registry_entry_t ChildEntry;
			while ((Desc.RegistryID == 0) && (ChildEntry = IOIteratorNext(ChildIterator)))
			{
				CFStringRef IOMatchCategory = (CFStringRef)IORegistryEntrySearchCFProperty(ChildEntry, kIOServicePlane, IOMatchCategoryRef, kCFAllocatorDefault, 0);
				if (IOMatchCategory && (CFGetTypeID(IOMatchCategory) == CFStringGetTypeID()) && (CFStringCompare(IOMatchCategory, IOAcceleratorRef, 0) == kCFCompareEqualTo))
				{
					CFMutableDictionaryRef Properties = nullptr;
					if (kIOReturnSuccess == IORegistryEntryCreateCFProperties(ChildEntry, &Properties, kCFAllocatorDefault, kIORegistryIterateRecursively))
					{
						kern_return_t Result = IORegistryEntryGetRegistryEntryID(ChildEntry, &Desc.RegistryID);
						check(Result == kIOReturnSuccess);

						CFStringRef BundleID = (CFStringRef)CFDictionaryGetValue(Properties, CFBundleIdentifier);
						if (BundleID && (CFGetTypeID(BundleID) == CFStringGetTypeID()))
						{
							Desc.GPUBundleID = [[NSString alloc] initWithString:(__bridge NSString*)BundleID];
						}

						{
							char Buffer[0x40] = { 0 };
							size_t BufferSize = sizeof(Buffer) / sizeof(Buffer[0]);
							if (0 == sysctlbyname("hw.targettype", &Buffer, &BufferSize, NULL, 0))
							{
								uint32_t Value = PLATFORM_MAC_MAKE_FOURCC(Buffer[0], Buffer[1], Buffer[2], Buffer[3]);
								Desc.GPUDeviceId = Value;
							}
						}

						CFDataRef VendorID = (CFDataRef)CFDictionaryGetValue(Properties, VendorIDRef);
						if (VendorID && (CFGetTypeID(VendorID) == CFDataGetTypeID()))
						{
							const uint32_t* Value = reinterpret_cast<const uint32_t*>(CFDataGetBytePtr(VendorID));
							Desc.GPUVendorId = *Value;
						}

						CFStringRef MetalPluginName = (CFStringRef)CFDictionaryGetValue(Properties, MetalPluginNameRef);
						if (MetalPluginName && (CFGetTypeID(MetalPluginName) == CFStringGetTypeID()))
						{
							Desc.GPUMetalBundle = [[NSString alloc] initWithString:(__bridge NSString*)MetalPluginName];
						}

						CFStringRef GLBundleName = (CFStringRef)CFDictionaryGetValue(Properties, GLBundleNameRef);
						if (GLBundleName && (CFGetTypeID(GLBundleName) == CFStringGetTypeID()))
						{
							Desc.GPUOpenGLBundle = [[NSString alloc] initWithString:(__bridge NSString*)GLBundleName];
						}

						CFStringRef ModelName = (CFStringRef)CFDictionaryGetValue(Properties, ModelRef);
						if (ModelName && (CFGetTypeID(ModelName) == CFStringGetTypeID()))
						{
							Desc.GPUName = [[NSString alloc] initWithString:(__bridge NSString*)ModelName];
						}

						{
							uint64_t Value = 0;
							size_t ValueSize = sizeof(Value);
							if (0 == sysctlbyname("hw.memsize", &Value, &ValueSize, NULL, 0))
							{
								Desc.GPUMemoryMB = uint64(float(Value) * 0.75f) / 1024 / 1024;
							}
						}
					}

					if (Properties)
					{
						CFRelease(Properties);
					}
				}

				if (IOMatchCategory)
				{
					CFRelease(IOMatchCategory);
				}

				IOObjectRelease(ChildEntry);
			}

			IOObjectRelease(ChildIterator);
		}
	}

#if PLATFORM_MAC_X86
	void InitializeDescriptorFromDeviceEntry(FMacGPUDescriptor& Desc, io_registry_entry_t ServiceEntry, CFMutableDictionaryRef ServiceInfo)
	{
		IOObjectRetain(ServiceEntry);
		Desc.PCIDevice = (uint32)ServiceEntry;
		
		static CFStringRef ModelRef = CFSTR("model");
		const CFDataRef Model = (const CFDataRef)CFDictionaryGetValue(ServiceInfo, ModelRef);
		if(Model)
		{
			if(CFGetTypeID(Model) == CFDataGetTypeID())
			{
				CFStringRef ModelName = CFStringCreateFromExternalRepresentation(kCFAllocatorDefault, Model, kCFStringEncodingASCII);
				
				Desc.GPUName = (NSString*)ModelName;
			}
			else
			{
				CFRelease(Model);
			}
		}
		
		static CFStringRef DeviceIDRef = CFSTR("device-id");
		const CFDataRef DeviceID = (const CFDataRef)CFDictionaryGetValue(ServiceInfo, DeviceIDRef);
		if(DeviceID && CFGetTypeID(DeviceID) == CFDataGetTypeID())
		{
			const uint32* Value = reinterpret_cast<const uint32*>(CFDataGetBytePtr(DeviceID));
			Desc.GPUDeviceId = *Value;
		}
		
		static CFStringRef VendorIDRef = CFSTR("vendor-id");
		const CFDataRef VendorID = (const CFDataRef)CFDictionaryGetValue(ServiceInfo, VendorIDRef);
		if(DeviceID && CFGetTypeID(DeviceID) == CFDataGetTypeID())
		{
			const uint32* Value = reinterpret_cast<const uint32*>(CFDataGetBytePtr(VendorID));
			Desc.GPUVendorId = *Value;
		}
		
		static CFStringRef HeadlessRef = CFSTR("headless");
		const CFBooleanRef Headless = (const CFBooleanRef)CFDictionaryGetValue(ServiceInfo, HeadlessRef);
		if(Headless && CFGetTypeID(Headless) == CFBooleanGetTypeID())
		{
			Desc.GPUHeadless = (bool)CFBooleanGetValue(Headless);
		}
		
		static CFStringRef VRAMTotal = CFSTR("VRAM,totalMB");
		CFTypeRef VRAM = IORegistryEntrySearchCFProperty(ServiceEntry, kIOServicePlane, VRAMTotal, kCFAllocatorDefault, kIORegistryIterateRecursively);
		if (VRAM)
		{
			if(CFGetTypeID(VRAM) == CFDataGetTypeID())
			{
				const uint32* Value = reinterpret_cast<const uint32*>(CFDataGetBytePtr((CFDataRef)VRAM));
				Desc.GPUMemoryMB = *Value;
			}
			else if(CFGetTypeID(VRAM) == CFNumberGetTypeID())
			{
				CFNumberGetValue((CFNumberRef)VRAM, kCFNumberSInt32Type, &Desc.GPUMemoryMB);
			}
			CFRelease(VRAM);
		}
		
		static CFStringRef MetalPluginName = CFSTR("MetalPluginName");
		const CFStringRef MetalLibName = (const CFStringRef)IORegistryEntrySearchCFProperty(ServiceEntry, kIOServicePlane, MetalPluginName, kCFAllocatorDefault, kIORegistryIterateRecursively);
		if(MetalLibName)
		{
			if(CFGetTypeID(MetalLibName) == CFStringGetTypeID())
			{
				Desc.GPUMetalBundle = (NSString*)MetalLibName;
			}
			else
			{
				CFRelease(MetalLibName);
			}
		}
		
		CFStringRef BundleID = nullptr;
		
		static CFStringRef CFBundleIdentifier = CFSTR("CFBundleIdentifier");
		
		io_iterator_t ChildIterator;
		if(IORegistryEntryGetChildIterator(ServiceEntry, kIOServicePlane, &ChildIterator) == kIOReturnSuccess)
		{
			io_registry_entry_t ChildEntry;
			while((BundleID == nullptr) && (ChildEntry = IOIteratorNext(ChildIterator)))
			{
				static CFStringRef IOMatchCategoryRef = CFSTR("IOMatchCategory");
				CFStringRef IOMatchCategory = (CFStringRef)IORegistryEntrySearchCFProperty(ChildEntry, kIOServicePlane, IOMatchCategoryRef, kCFAllocatorDefault, 0);
				static CFStringRef IOAcceleratorRef = CFSTR("IOAccelerator");
				if (IOMatchCategory && CFGetTypeID(IOMatchCategory) == CFStringGetTypeID() && CFStringCompare(IOMatchCategory, IOAcceleratorRef, 0) == kCFCompareEqualTo)
				{
					BundleID = (CFStringRef)IORegistryEntrySearchCFProperty(ChildEntry, kIOServicePlane, CFBundleIdentifier, kCFAllocatorDefault, 0);
					
					kern_return_t Result = IORegistryEntryGetRegistryEntryID(ChildEntry, &Desc.RegistryID);
					check(Result == kIOReturnSuccess);
				}
				if (IOMatchCategory)
				{
					CFRelease(IOMatchCategory);
				}
				IOObjectRelease(ChildEntry);
			}
			
			IOObjectRelease(ChildIterator);
		}
		
		if (BundleID == nullptr)
		{
			BundleID = (CFStringRef)IORegistryEntrySearchCFProperty(ServiceEntry, kIOServicePlane, CFBundleIdentifier, kCFAllocatorDefault, kIORegistryIterateRecursively);
		}
		
		if(BundleID)
		{
			if(CFGetTypeID(BundleID) == CFStringGetTypeID())
			{
				Desc.GPUBundleID = (NSString*)BundleID;
			}
			else
			{
				CFRelease(BundleID);
			}
		}
	}
#endif // PLATFORM_MAC_X86
    
public:
	static FMacPlatformGPUManager& Get()
	{
		static FMacPlatformGPUManager sSelf;
		return sSelf;
	}

	FMacPlatformGPUManager()
	{
		FScopeLock Lock(&Mutex);

		CFStringRef ClassCodeRef = CFStringCreateWithCString(kCFAllocatorDefault, GetClassCode(), kCFStringEncodingUTF8);
		check(ClassCodeRef);

		// Enumerate the GPUs via IOKit to avoid dragging in OpenGL
		io_iterator_t Iterator;
		CFMutableDictionaryRef MatchDictionary = IOServiceMatching(GetIOServiceMatchingName());
		if(IOServiceGetMatchingServices(kIOMainPortDefault, MatchDictionary, &Iterator) == kIOReturnSuccess)
		{
			uint32 Index = 0;
			io_registry_entry_t ServiceEntry;
			while((ServiceEntry = IOIteratorNext(Iterator)))
			{
				CFMutableDictionaryRef ServiceInfo;
				if(IORegistryEntryCreateCFProperties(ServiceEntry, &ServiceInfo, kCFAllocatorDefault, kNilOptions) == kIOReturnSuccess)
				{
					const CFDataRef ClassCode = (const CFDataRef)CFDictionaryGetValue(ServiceInfo, ClassCodeRef);
					if(ClassCode && CFGetTypeID(ClassCode) == CFDataGetTypeID())
					{
						if (IsRunningOnAppleSilicon())
						{
							const char* ClassCodeValue = reinterpret_cast<const char*>(CFDataGetBytePtr(ClassCode));
							if (!strncasecmp(ClassCodeValue, "sgx", 3))
							{
								FMacGPUDescriptor Desc;
#if PLATFORM_MAC_X86
								// Default initialized to 0 on ARM64, but initialized here for X86 due to
								// 4.26.0 compatibility requirements.
								Desc.RegistryID = 0;
#endif // PLATFORM_MAC_X86
								InitializeDescriptorFromDeviceEntryM(Desc, ServiceEntry, ServiceInfo);
								if (Desc.GPUMetalBundle)
								{
									Desc.GPUIndex = Index++;
									CurrentGPUs.Add(Desc);
								}
							}
						}
#if PLATFORM_MAC_X86
						else
						{
							const uint32* ClassCodeValue = reinterpret_cast<const uint32*>(CFDataGetBytePtr(ClassCode));

							// GPUs are class-code 0x30000 || 0x38000
							if (ClassCodeValue && (*ClassCodeValue == 0x30000 || *ClassCodeValue == 0x38000))
							{
								FMacGPUDescriptor Desc;

								InitializeDescriptorFromDeviceEntry(Desc, ServiceEntry, ServiceInfo);

								if (Desc.GPUMetalBundle)
								{
									Desc.GPUIndex = Index++;

									CurrentGPUs.Add(Desc);
								}
							}
						}
#endif // PLATFORM_MAC_X86
					}
					CFRelease(ServiceInfo);
				}
				IOObjectRelease(ServiceEntry);
			}
			IOObjectRelease(Iterator);
		}

		CFRelease(ClassCodeRef);

		UpdatedGPUs = CurrentGPUs;
	}
	
	TArray<FMacGPUDescriptor> const& GetCurrentGPUs()
	{
		if (bRequiresUpdate)
		{
			FScopeLock Lock(&Mutex);
			CurrentGPUs = UpdatedGPUs;
			bRequiresUpdate = false;
		}
		return CurrentGPUs;
	}
	
	void Notify(uint64_t DeviceRegistryID, FMacPlatformMisc::EMacGPUNotification Notification)
	{
#if PLATFORM_MAC_X86
		switch (Notification)
		{
			case FMacPlatformMisc::EMacGPUNotification::Added:
			{
				CFMutableDictionaryRef MatchDictionary = IORegistryEntryIDMatching(DeviceRegistryID);
				if(MatchDictionary)
				{
					io_registry_entry_t ServiceEntry = IOServiceGetMatchingService(kIOMainPortDefault, MatchDictionary);
					if(ServiceEntry)
					{
						io_iterator_t ParentIterator;
						if(IORegistryEntryGetParentIterator(ServiceEntry, kIOServicePlane, &ParentIterator) == kIOReturnSuccess)
						{
							io_registry_entry_t ParentEntry;
							while((ParentEntry = IOIteratorNext(ParentIterator)))
							{
								CFMutableDictionaryRef ServiceInfo;
								if(IORegistryEntryCreateCFProperties(ParentEntry, &ServiceInfo, kCFAllocatorDefault, kNilOptions) == kIOReturnSuccess)
								{
									// GPUs are class-code 0x30000 || 0x38000
									static CFStringRef ClassCodeRef = CFSTR("class-code");
									const CFDataRef ClassCode = (const CFDataRef)CFDictionaryGetValue(ServiceInfo, ClassCodeRef);
									if(ClassCode && CFGetTypeID(ClassCode) == CFDataGetTypeID())
									{
										const uint32* ClassCodeValue = reinterpret_cast<const uint32*>(CFDataGetBytePtr(ClassCode));
										if(ClassCodeValue && (*ClassCodeValue == 0x30000 || *ClassCodeValue == 0x38000))
										{
											FScopeLock Lock(&Mutex);
											
											FMacGPUDescriptor Desc;
											
                                            InitializeDescriptorFromDeviceEntry(Desc, ServiceEntry, ServiceInfo);
											
											if (Desc.GPUMetalBundle)
											{
												Desc.GPUIndex = UpdatedGPUs.Num();
												
												UpdatedGPUs.Add(Desc);
											}
											
											bRequiresUpdate = true;
											break;
										}
									}
									CFRelease(ServiceInfo);
								}
								IOObjectRelease(ParentEntry);
							}
							IOObjectRelease(ParentIterator);
						}
						IOObjectRelease(ServiceEntry);
					}
				}
				break;
			}
			case FMacPlatformMisc::EMacGPUNotification::RemovalRequested:
			case FMacPlatformMisc::EMacGPUNotification::Removed:
			{
				FScopeLock Lock(&Mutex);
				for (int32 i = 0; i < UpdatedGPUs.Num(); i++)
				{
					FMacGPUDescriptor& Desc = UpdatedGPUs[i];
					if (Desc.RegistryID == DeviceRegistryID)
					{
						if (Desc.GPUIndex == GMacExplicitRendererID)
						{
							GMacExplicitRendererID = -1;
						}
						UpdatedGPUs.RemoveAt(i);
						break;
					}
				}
				for (int32 i = 0; i < UpdatedGPUs.Num(); i++)
				{
					FMacGPUDescriptor& Desc = UpdatedGPUs[i];
					Desc.GPUIndex = (uint32)i;
				}
				bRequiresUpdate = true;
				break;
			}
			default:
			{
				break;
			}
		}
#endif // PLATFORM_MAC_X86
	}
};

void FMacPlatformMisc::GPUChangeNotification(uint64_t DeviceRegistryID, EMacGPUNotification Notification)
{
	FMacPlatformGPUManager::Get().Notify(DeviceRegistryID, Notification);
}

TArray<FMacGPUDescriptor> const& FMacPlatformMisc::GetGPUDescriptors()
{
	return FMacPlatformGPUManager::Get().GetCurrentGPUs();
}

int32 FMacPlatformMisc::GetExplicitRendererIndex()
{
	check(GConfig && GConfig->IsReadyForUse());
	
	int32 ExplicitRenderer = -1;
	if (GMacExplicitRendererID == -1 && (FParse::Value(FCommandLine::Get(),TEXT("MacExplicitRenderer="), ExplicitRenderer) && ExplicitRenderer >= 0))
	{
		GMacExplicitRendererID = ExplicitRenderer;
	}
	else if (GMacExplicitRendererID == -1 && (GConfig->GetInt(MAC_GRAPHICS_SETTINGS, TEXT("RendererID"), ExplicitRenderer, MAC_GRAPHICS_INI) && ExplicitRenderer >= 0))
	{
		GMacExplicitRendererID = ExplicitRenderer;
	}
	
	return GMacExplicitRendererID;
}

FString FMacPlatformMisc::GetPrimaryGPUBrand()
{
	static FString PrimaryGPU;
	if(PrimaryGPU.IsEmpty())
	{
		TArray<FMacGPUDescriptor> const& GPUs = GetGPUDescriptors();
		
		if(GPUs.Num() > 1)
		{
			for(FMacGPUDescriptor const& GPU : GPUs)
			{
				if(!GPU.GPUHeadless && GPU.GPUVendorId != 0x8086)
				{
					PrimaryGPU = GPU.GPUName;
					break;
				}
			}
		}
		
		if ( PrimaryGPU.IsEmpty() && GPUs.Num() > 0 )
		{
			PrimaryGPU = GPUs[0].GPUName;
		}
			
			if ( PrimaryGPU.IsEmpty() )
			{
				PrimaryGPU = FGenericPlatformMisc::GetPrimaryGPUBrand();
			}
		}
	return PrimaryGPU;
	}

FGPUDriverInfo FMacPlatformMisc::GetGPUDriverInfo(const FString& DeviceDescription, bool bVerbose)
{
	SCOPED_AUTORELEASE_POOL;
	
	if (bVerbose)
	{
		// Also report monitor information here, for lack of a better place.
		UE_LOGF(LogMac, Log, "Attached monitors:");
		int NumMonitors = 0;
		
		for (NSScreen* Screen in NSScreen.screens)
		{
			NSDictionary* ScreenDescription = Screen.deviceDescription;
			CGSize PointSize = Screen.frame.size;
			NSSize PixelSize = [[ScreenDescription objectForKey:NSDeviceSize] sizeValue];
			NSRect WorkArea = Screen.visibleFrame;
			NSNumber* Scale = ScreenDescription[NSDeviceResolution];
			
			UE_LOGF(LogMac, Log, "Screen: %ls, Resolution: %.0fx%.0f, Points: %.0fx%.0f, WorkArea: (%.0f, %.0f) -> (%.0f, %.0f), Scale: %.2f"
				   , Screen.localizedName ? *FString((__bridge CFStringRef)Screen.localizedName) : TEXT("(Unnamed)")
				   , PixelSize.width, PixelSize.height
				   , PointSize.width, PointSize.height
				   , WorkArea.origin.x, WorkArea.origin.y, WorkArea.origin.x + WorkArea.size.width, WorkArea.origin.y + WorkArea.size.height
				   , Screen.backingScaleFactor);
			NumMonitors++;
		}
		
		UE_LOGF(LogMac, Log, "Found %d attached monitors.", NumMonitors);
	}

	FGPUDriverInfo Info;
	TArray<FString> NameComponents;
	TArray<FMacGPUDescriptor> const& GPUs = GetGPUDescriptors();
	
	for(FMacGPUDescriptor const& GPU : GPUs)
	{
		NameComponents.Empty();
		bool bMatchesName = FString(GPU.GPUName).TrimStart().ParseIntoArray(NameComponents, TEXT(" ")) > 0;
		for (FString& Component : NameComponents)
		{
			bMatchesName &= DeviceDescription.Contains(Component);
		}
		
		if (bMatchesName)
		{
			Info.VendorId = GPU.GPUVendorId;
			Info.DeviceDescription = FString(GPU.GPUName);

			if (Info.IsAMD())
			{
				Info.ProviderName = TEXT("AMD");
			}
			else if (Info.IsIntel())
			{
				Info.ProviderName = TEXT("Intel");
			}
			else
			{
				Info.ProviderName = TEXT("Apple");
			}
			
			bool bGotInternalVersionInfo = false;
			bool bGotUserVersionInfo = false;
			bool bGotDate = false;

			for(uint32 Index = 0; Index < _dyld_image_count(); Index++)
			{
				char const* IndexName = _dyld_get_image_name(Index);
				FString FullModulePath(IndexName);
				FString Name = FPaths::GetBaseFilename(FullModulePath);
				if(Name == FString(GPU.GPUMetalBundle) || Name == FString(GPU.GPUOpenGLBundle))
				{
					struct mach_header_64 const* IndexModule64 = NULL;
					struct load_command const* LoadCommands = NULL;
					
					struct mach_header const* IndexModule32 = _dyld_get_image_header(Index);
					check(IndexModule32->magic == MH_MAGIC_64);
					
					IndexModule64 = (struct mach_header_64 const*)IndexModule32;
					LoadCommands = (struct load_command const*)(IndexModule64 + 1);
					struct load_command const* Command = LoadCommands;
					struct dylib_command const* DylibID = nullptr;
					struct source_version_command const* SourceVersion = nullptr;
					for(uint32 CommandIndex = 0; CommandIndex < IndexModule64->ncmds; CommandIndex++)
					{
						if (Command && Command->cmd == LC_ID_DYLIB)
						{
							DylibID = (struct dylib_command const*)Command;
							break;
						}
						else if(Command && Command->cmd == LC_SOURCE_VERSION)
						{
							SourceVersion = (struct source_version_command const*)Command;
						}
						Command = (struct load_command const*)(((char const*)Command) + Command->cmdsize);
					}
					if(DylibID)
					{
						uint32 Major = ((DylibID->dylib.current_version >> 16) & 0xffff);
						uint32 Minor = ((DylibID->dylib.current_version >> 8) & 0xff);
						uint32 Patch = ((DylibID->dylib.current_version & 0xff));
						Info.InternalDriverVersion = FString::Printf(TEXT("%d.%d.%d"), Major, Minor, Patch);
						
						time_t DylibTime = (time_t)DylibID->dylib.timestamp;
						struct tm Time;
						gmtime_r(&DylibTime, &Time);
						Info.DriverDate = FString::Printf(TEXT("%d-%d-%d"), Time.tm_mon + 1, Time.tm_mday, 1900 + Time.tm_year);

						bGotInternalVersionInfo = Major != 0 || Minor != 0 || Patch != 0;
						bGotDate = (1900 + Time.tm_year) >= 2014;
						break;
					}
					else if (SourceVersion)
					{
						uint32 A = ((SourceVersion->version >> 40) & 0xffffff);
						uint32 B = ((SourceVersion->version >> 30) & 0x3ff);
						uint32 C = ((SourceVersion->version >> 20) & 0x3ff);
						uint32 D = ((SourceVersion->version >> 10) & 0x3ff);
						uint32 E = (SourceVersion->version & 0x3ff);
						Info.InternalDriverVersion = FString::Printf(TEXT("%d.%d.%d.%d.%d"), A, B, C, D, E);
						
						struct stat Stat;
						stat(IndexName, &Stat);
						
						struct tm Time;
						gmtime_r(&Stat.st_mtime, &Time);
						Info.DriverDate = FString::Printf(TEXT("%d-%d-%d"), Time.tm_mon + 1, Time.tm_mday, 1900 + Time.tm_year);
						
						bGotInternalVersionInfo = A != 0 || B != 0 || C != 0 || D != 0;
						bGotDate = (1900 + Time.tm_year) >= 2014;
					}
				}
			}

			bool bCanPullDriverInfo = !GMacAppInfo.bIsSandboxed;

			if(bCanPullDriverInfo)
			{
				if(!bGotDate || !bGotInternalVersionInfo || !bGotUserVersionInfo)
				{
					NSURL* URL = (NSURL*)KextManagerCreateURLForBundleIdentifier(kCFAllocatorDefault, (CFStringRef)GPU.GPUBundleID);
					if(URL)
					{
						NSBundle* ControllerBundle = [NSBundle bundleWithURL:URL];
						if(ControllerBundle)
						{
							NSDictionary* Dict = ControllerBundle.infoDictionary;
							NSString* BundleVersion = [Dict objectForKey:@"CFBundleVersion"];
							NSString* BundleShortVersion = [Dict objectForKey:@"CFBundleShortVersionString"];
							NSString* BundleInfoVersion = [Dict objectForKey:@"CFBundleGetInfoString"];
							if (!bGotInternalVersionInfo && (BundleVersion || BundleShortVersion))
							{
								Info.InternalDriverVersion = FString(BundleShortVersion ? BundleShortVersion : BundleVersion);
								bGotInternalVersionInfo = true;
							}
							if (!bGotUserVersionInfo && BundleInfoVersion)
							{
								Info.UserDriverVersion = FString(BundleInfoVersion);
								bGotUserVersionInfo = true;
							}
							
							if(!bGotDate)
							{
								NSURL* Exe = ControllerBundle.executableURL;
								if (Exe)
								{
									id Value = nil;
									if([Exe getResourceValue:&Value forKey:NSURLContentModificationDateKey error:nil] && Value)
									{
										NSDate* Date = (NSDate*)Value;
										Info.DriverDate = [Date descriptionWithLocale:nil];
										bGotDate = true;
									}
								}
							}
						}
						[URL release];
					}
				}
				
				if(!bGotInternalVersionInfo)
				{
					NSArray* Array = [NSArray arrayWithObject: GPU.GPUBundleID];
					NSDictionary* Dict = (NSDictionary*)KextManagerCopyLoadedKextInfo((CFArrayRef)Array, nil);
					if(Dict)
					{
						NSDictionary* ControllerDict = [Dict objectForKey:GPU.GPUBundleID];
						if(ControllerDict)
						{
							NSString* BundleVersion = [ControllerDict objectForKey:@"CFBundleVersion"];
							Info.InternalDriverVersion = FString(BundleVersion);
						}
						[Dict release];
					}
				}
			}
			else if(bGotInternalVersionInfo && !bGotUserVersionInfo)
			{
				Info.UserDriverVersion = Info.InternalDriverVersion;
			}
			
			break;
		}
	}
	
	return Info;
}

void FMacPlatformMisc::GetOSVersions( FString& out_OSVersionLabel, FString& out_OSSubVersionLabel )
{
	MacPlatformGetOSProductVersion(out_OSVersionLabel);
	out_OSVersionLabel = FString("macOS ") + out_OSVersionLabel;
	MacPlatformGetOSVersion(out_OSSubVersionLabel);
}

FString FMacPlatformMisc::GetOSVersion()
{
	FString OSVersion;
	MacPlatformGetOSProductVersion(OSVersion);
	return OSVersion;
}

NSOperatingSystemVersion FMacPlatformMiscEx::GetNSOperatingSystemVersion()
{
	FString OSVersionString;
	NSOperatingSystemVersion SystemVersion = {};
	MacPlatformGetOSProductVersion(OSVersionString);
	if (sscanf(TCHAR_TO_ANSI(*OSVersionString), "%ld.%ld.%ld", &SystemVersion.majorVersion, &SystemVersion.minorVersion, &SystemVersion.patchVersion))
	{
		return SystemVersion;
	}
	else
	{
		return {0,0,0};
	}
}

bool FMacPlatformMisc::GetDiskTotalAndFreeSpace(const FString& InPath, uint64& TotalNumberOfBytes, uint64& NumberOfFreeBytes)
{
	struct statfs FSStat = { 0 };
	FTCHARToUTF8 Converter(*InPath);
	int Err = statfs((ANSICHAR*)Converter.Get(), &FSStat);
	if (Err == 0)
	{
		TotalNumberOfBytes = FSStat.f_blocks * FSStat.f_bsize;
		NumberOfFreeBytes = FSStat.f_bavail * FSStat.f_bsize;
	}
	else
	{
		int ErrNo = errno;
		UE_LOGF(LogMac, Warning, "Unable to statfs('%ls'): errno=%d (%ls)", *InPath, ErrNo, UTF8_TO_TCHAR(strerror(ErrNo)));
	}
	return (Err == 0);
}

bool FMacPlatformMisc::HasSeparateChannelForDebugOutput()
{
	return FPlatformMisc::IsDebuggerPresent() || isatty(STDOUT_FILENO) || isatty(STDERR_FILENO);
}

FString FMacPlatformMisc::GetBonjourName()
{
	static FString BonjourName;
	if (BonjourName.IsEmpty())
	{
		CFStringRef LocalHostNameRef = SCDynamicStoreCopyLocalHostName(NULL);
		if (LocalHostNameRef != nullptr)
		{
			// Per the Bonjour docs, https://developer.apple.com/library/archive/documentation/Cocoa/Conceptual/NetServices/Articles/domainnames.html
			// the maximum length of the host name and of the fully qualified domain name (FQDN) is 63 bytes per label and 255 bytes per FQDN.
			TCHAR Result[256] = TEXT("");
			FPlatformString::CFStringToTCHAR(LocalHostNameRef, Result);
			BonjourName = FString(Result);
			CFRelease(LocalHostNameRef);
		}
	}

	return BonjourName;
}

FString FMacPlatformMisc::GetCPUVendor()
{
#if PLATFORM_MAC_ARM64
    return TEXT("Apple");
#else
	union
	{
		char Buffer[12+1];
		struct
		{
			int dw0;
			int dw1;
			int dw2;
		} Dw;
	} VendorResult;


	int32 Args[4];
	asm( "cpuid" : "=a" (Args[0]), "=b" (Args[1]), "=c" (Args[2]), "=d" (Args[3]) : "a" (0));

	VendorResult.Dw.dw0 = Args[1];
	VendorResult.Dw.dw1 = Args[3];
	VendorResult.Dw.dw2 = Args[2];
	VendorResult.Buffer[12] = 0;

	return ANSI_TO_TCHAR(VendorResult.Buffer);
#endif
}

FString FMacPlatformMisc::GetCPUBrand()
{
	static FString Result = FGenericPlatformMisc::GetCPUBrand();
	static bool bHaveResult = false;

	if (!bHaveResult)
	{
		// @see (x86_64) for more information http://msdn.microsoft.com/en-us/library/vstudio/hskdteyh(v=vs.100).aspx
		ANSICHAR BrandString[0x40] = { 0 };

		if (IsRunningOnAppleSilicon())
		{
			size_t BrandStringLength = sizeof(BrandString) / sizeof(BrandString[0]);
			sysctlbyname("machdep.cpu.brand_string", &BrandString, &BrandStringLength, NULL, 0);
		}
#if PLATFORM_MAC_X86
		else
		{
			int32 CPUInfo[4] = { -1 };
			const SIZE_T CPUInfoSize = sizeof(CPUInfo);

			asm( "cpuid" : "=a" (CPUInfo[0]), "=b" (CPUInfo[1]), "=c" (CPUInfo[2]), "=d" (CPUInfo[3]) : "a" (0x80000000));
			const uint32 MaxExtIDs = CPUInfo[0];

			if (MaxExtIDs >= 0x80000004)
			{
				const uint32 FirstBrandString = 0x80000002;
				const uint32 NumBrandStrings = 3;
				for (uint32 Index = 0; Index < NumBrandStrings; ++Index)
				{
					asm( "cpuid" : "=a" (CPUInfo[0]), "=b" (CPUInfo[1]), "=c" (CPUInfo[2]), "=d" (CPUInfo[3]) : "a" (FirstBrandString + Index));
					FPlatformMemory::Memcpy(BrandString + CPUInfoSize * Index, CPUInfo, CPUInfoSize);
				}
			}
		}
#endif // PLATFORM_MAC_X86

		Result = BrandString;

		bHaveResult = true;
	}

	return FString(Result);
}

uint32 FMacPlatformMisc::GetCPUInfo()
{
#if PLATFORM_MAC_ARM64
    // not implenented, this is an optional function
    return FGenericPlatformMisc::GetCPUInfo();
#else
	uint32 Args[4];
	asm( "cpuid" : "=a" (Args[0]), "=b" (Args[1]), "=c" (Args[2]), "=d" (Args[3]) : "a" (1));

	return Args[0];
#endif
}

FText FMacPlatformMisc::GetFileManagerName()
{
	return NSLOCTEXT("MacPlatform", "FileManagerName", "Finder");
}

bool FMacPlatformMisc::IsRunningOnBattery()
{
	return GMacAppInfo.RunningOnBattery;
}

int32 FMacPlatformMiscEx::VersionCompare(const NSOperatingSystemVersion& VersionA, const NSOperatingSystemVersion& VersionB)
{
	NSInteger ValuesA[3] = { VersionA.majorVersion, VersionA.minorVersion, VersionA.patchVersion };
	NSInteger ValuesB[3] = { VersionB.majorVersion, VersionB.minorVersion, VersionB.patchVersion };

	for (uint32 i = 0; i < 3; i++)
	{
		if (ValuesA[i] < ValuesB[i])
		{
			return -1;
		}
		else if (ValuesA[i] > ValuesB[i])
		{
			return 1;
		}
	}

	return 0;
}

int32 FMacPlatformMisc::MacOSXVersionCompare(uint8 Major, uint8 Minor, uint8 Revision)
{
	return FMacPlatformMiscEx::VersionCompare(GMacAppInfo.OSXVersion, {Major, Minor, Revision});
}

FString FMacPlatformMisc::GetOperatingSystemId()
{
	FString Result;
	io_service_t Entry = IOServiceGetMatchingService(kIOMainPortDefault, IOServiceMatching("IOPlatformExpertDevice"));
	if (Entry)
	{
		CFTypeRef UUID = IORegistryEntryCreateCFProperty(Entry, CFSTR(kIOPlatformUUIDKey), kCFAllocatorDefault, 0);
		Result = FString((__bridge NSString*)UUID);
		IOObjectRelease(Entry);
		CFRelease(UUID);
	}
	else
	{
		UE_LOGF(LogMac, Warning, "GetOperatingSystemId() failed");
	}
	return Result;
}

FString FMacPlatformMisc::GetXcodePath()
{
	return GMacAppInfo.XcodePath;
}

int32 FMacPlatformMisc::XCodeVersionCompare(uint8 Major, uint8 Minor, uint8 Revision)
{
	return FMacPlatformMiscEx::VersionCompare(GMacAppInfo.XcodeVersion, {Major, Minor, Revision});
}

bool FMacPlatformMisc::IsSupportedXcodeVersionInstalled()
{
	// We need Xcode 9.4 or newer to be able to compile Metal shaders correctly
	return GMacAppInfo.XcodeVersion.majorVersion > 9 || (GMacAppInfo.XcodeVersion.majorVersion == 9 && GMacAppInfo.XcodeVersion.minorVersion >= 4);
}

#if WITH_EDITOR
struct FMacModel
{
	NSString*	Name;
	uint32		Major;
	uint32		Minor;
	bool		bIsLowPower;
};

bool FMacPlatformMisc::IsRunningOnRecommendedMinSpecHardware()
{
	const TArray<FMacModel> MinSupportedMacModels(
	{
		{ @"MacPro",		6,	1,	false },	// Mac Pro (Late 2013) [AMD FirePro D300] { Pitcairn XT GL (GFX6) }
		{ @"Macmini",		7,	1,	false },	// Mac mini (Late 2014) [Intel HD Graphics 5000] { Haswell GT3 }
		{ @"MacBookPro",	11,	4,	false },	// MacBook Pro (Retina, 15-inch, Mid 2015) [Intel Iris Pro 5200] { Haswell GT3e }
		{ @"MacBookAir",	6,	1,	true  },	// MacBook Air (11-inch, Mid 2013) [Intel HD Graphics 5000] { Haswell GT3 }
		{ @"MacBook",		8,	1,	true  },	// MacBook (Retina, 12-inch, Early 2015) [Intel HD Graphics 5300] { Broadwell GT2 }
		{ @"iMacPro",		1,	1,	false },	// iMac Pro (Retina 5K, 27-inch, 2017) [AMD Radeon Pro Vega 56/64/64X] { GFX9 }
		{ @"iMac",			14,	4,	false },	// iMac (21.5-inch, Mid 2014) [Intel HD Graphics 5000] { Haswell GT3 }
	});

	constexpr int64 MinSupportedMemsize = 8ll * (1024 * 1024 * 1024);

	int64	SystemMemsize		= 0;
	SIZE_T	SystemMemsizeLen	= sizeof(SystemMemsize);

	// Fetch memsize
	sysctlbyname("hw.memsize", &SystemMemsize, &SystemMemsizeLen, NULL, 0);

	// Check memsize first
	bool bSupported = (SystemMemsize >= MinSupportedMemsize);
	if (bSupported)
	{
		// Fetch model
		ANSICHAR	SystemModel[20]		= { '\0' };
		SIZE_T		SystemModelLen		= sizeof(SystemModel);
		ANSICHAR	SystemModelName[20]	= { '\0' };
		uint32		SystemModelMajor	= 0;
		uint32		SystemModelMinor	= 0;

		sysctlbyname("hw.model", SystemModel, &SystemModelLen, NULL, 0);
		sscanf(SystemModel, "%[^0-9]%u,%u", SystemModelName, &SystemModelMajor, &SystemModelMinor);

		NSString* ModelName = [[NSString alloc] initWithBytesNoCopy:(void*)SystemModelName
															 length:strlen(SystemModelName)
														   encoding:NSUTF8StringEncoding
													   freeWhenDone:NO];

		// Check model next, assume unknown models are OK.
		int32 Index = MinSupportedMacModels.IndexOfByPredicate([ModelName](const FMacModel& Other){ return (NSOrderedSame == [ModelName caseInsensitiveCompare:Other.Name]); });
		if (INDEX_NONE != Index)
		{
			const FMacModel& Model = MinSupportedMacModels[Index];

			if ( (SystemModelMajor <  Model.Major) ||
				((SystemModelMajor == Model.Major) && (SystemModelMinor < Model.Minor)))
			{
				bSupported = false;
			}

			// MacBook and MacBookAir are not high-powered machines, but Apple
			// Silicon is sufficient for running the editor.
			if (bSupported && Model.bIsLowPower)
			{
				int32	SystemArm64		= 0;
				SIZE_T	SystemArm64Len	= sizeof(SystemArm64);
				sysctlbyname("hw.optional.arm64", &SystemArm64, &SystemArm64Len, NULL, 0);

				bSupported = (SystemArm64 != 0);
			}
		}

		[ModelName release];
	}

	return bSupported;
}
#endif // WITH_EDITOR

CGDisplayModeRef FMacPlatformMiscEx::GetSupportedDisplayMode(CGDirectDisplayID DisplayID, uint32 Width, uint32 Height)
{
	CGDisplayModeRef BestMatchingMode = nullptr;
	uint32 BestWidth = 0;
	uint32 BestHeight = 0;

	CFArrayRef AllModes = CGDisplayCopyAllDisplayModes(DisplayID, nullptr);
	if (AllModes)
	{
		const int32 NumModes = CFArrayGetCount(AllModes);
		for (int32 Index = 0; Index < NumModes; Index++)
		{
			CGDisplayModeRef Mode = (CGDisplayModeRef)CFArrayGetValueAtIndex(AllModes, Index);
			const int32 ModeWidth = (int32)CGDisplayModeGetWidth(Mode);
			const int32 ModeHeight = (int32)CGDisplayModeGetHeight(Mode);

			const bool bIsEqualOrBetterWidth = FMath::Abs((int32)ModeWidth - (int32)Width) <= FMath::Abs((int32)BestWidth - (int32)Width);
			const bool bIsEqualOrBetterHeight = FMath::Abs((int32)ModeHeight - (int32)Height) <= FMath::Abs((int32)BestHeight - (int32)Height);
			if (!BestMatchingMode || (bIsEqualOrBetterWidth && bIsEqualOrBetterHeight))
			{
				BestWidth = ModeWidth;
				BestHeight = ModeHeight;
				BestMatchingMode = Mode;
			}
		}
		BestMatchingMode = CGDisplayModeRetain(BestMatchingMode);
		CFRelease(AllModes);
	}

	return BestMatchingMode;
}

/** Global pointer to crash handler */
void (* GCrashHandlerPointer)(const FGenericCrashContext& Context) = NULL;

/**
 * Good enough default crash reporter.
 */
static void DefaultCrashHandler(FMacCrashContext const& Context)
{
	Context.ReportCrash();
	if (GLog)
	{
		GLog->Panic();
	}
	if (GWarn)
	{
		GWarn->Flush();
	}
	if (GError)
	{
		GError->Flush();
		GError->HandleError();
	}
	
	return Context.GenerateCrashInfoAndLaunchReporter();
}

/** Message for the assert triggered on this thread */
thread_local const TCHAR* GCrashErrorMessage = nullptr;
thread_local void* GCrashErrorProgramCounter = nullptr;
thread_local ECrashContextType GCrashErrorType = ECrashContextType::Crash;
thread_local uint8* GCrashContextMemory[sizeof(FMacCrashContext)];

/** True system-specific crash handler that gets called first */
static void PlatformCrashHandler(int32 Signal, siginfo_t* Info, void* Context)
{
	// Disable CoreSymbolication
	FApplePlatformSymbolication::EnableCoreSymbolication( false );

	ECrashContextType Type;
	const TCHAR* ErrorMessage;
	void* ErrorProgramCounter;

	if (GCrashErrorMessage == nullptr)
	{
		Type = ECrashContextType::Crash;
		ErrorMessage = TEXT("Caught signal");
		ErrorProgramCounter = Info->si_addr;
	}
	else
	{
		Type = GCrashErrorType;
		ErrorMessage = GCrashErrorMessage;
		ErrorProgramCounter = GCrashErrorProgramCounter;
	}

	FMacCrashContext* CrashContext = new (GCrashContextMemory) FMacCrashContext(Type, ErrorMessage);
	CrashContext->InitFromSignal(Signal, Info, Context);
	CrashContext->ErrorFrame = ErrorProgramCounter;

	// Switch to crash handler malloc to avoid malloc reentrancy
	check(GCrashMalloc);
	GCrashMalloc->Enable(CrashContext, FPlatformTLS::GetCurrentThreadId());

	if (GCrashHandlerPointer)
	{
		GCrashHandlerPointer(*CrashContext);
	}
	else
	{
		// call default one
		DefaultCrashHandler(*CrashContext);
	}
}

static void PLCrashReporterHandler(siginfo_t* Info, ucontext_t* Uap, void* Context)
{
	if (Info->si_signo == SIGUSR2)
	{
		// All of these are locked on a mutex from where the SIGUSR2 signal was raised. Only touch these here in the signal handler
		extern ANSICHAR* GThreadCallStack;
		extern uint64* GThreadBackTrace;
		extern SIZE_T GThreadCallStackSize;
		extern bool GThreadCallStackInUse;
		extern uint32 GThreadBackTraceCount;

		// Only handle this if we have a valid plcrashreporter context. As backtrace(...) does not work in a signal handler when
		// an alternative stack is used
		if (FMacApplicationInfo::CrashReporter)
		{
			if (GThreadCallStack)
			{
				FPlatformStackWalk::StackWalkAndDump(GThreadCallStack, GThreadCallStackSize, 0, FMacApplicationInfo::CrashReporter);
			}
			else if  (GThreadBackTrace)
			{
				GThreadBackTraceCount = FPlatformStackWalk::CaptureStackBackTrace(GThreadBackTrace, GThreadCallStackSize, FMacApplicationInfo::CrashReporter);
			}
		}

		GThreadCallStackInUse = false;
	}
	else
	{
		PlatformCrashHandler((int32)Info->si_signo, Info, Uap);
	}
}

/**
 * Handles graceful termination. Gives time to exit gracefully, but second signal will quit immediately.
 */
static void GracefulTerminationHandler(int32 Signal, siginfo_t* Info, void* Context)
{
	// make sure as much data is written to disk as possible
	if (GLog)
	{
		GLog->Panic();
	}
	if (GWarn)
	{
		GWarn->Flush();
	}
	if (GError)
	{
		GError->Flush();
	}
	
	if( !IsEngineExitRequested() )
	{
		RequestEngineExit(TEXT("Mac GracefulTerminationHandler"));
	}
	else
	{
		_Exit(1);
	}
}

void FMacPlatformMisc::SetGracefulTerminationHandler()
{
	struct sigaction Action;
	FMemory::Memzero(&Action, sizeof(struct sigaction));
	Action.sa_sigaction = GracefulTerminationHandler;
	sigemptyset(&Action.sa_mask);
	Action.sa_flags = SA_SIGINFO | SA_RESTART | SA_ONSTACK;
	sigaction(SIGINT, &Action, NULL);
	sigaction(SIGTERM, &Action, NULL);
	sigaction(SIGHUP, &Action, NULL);	//  this should actually cause the server to just re-read configs (restart?)
}

void FMacPlatformMisc::SetCrashHandler(void (* CrashHandler)(const FGenericCrashContext& Context))
{
	SCOPED_AUTORELEASE_POOL;
	
	GCrashHandlerPointer = CrashHandler;
	
	bool bEnableCustomSignalHandling = true;
	// For packaged games, only use custom signal handling when CRC is opt-in. Otherwise let macOS process crash without UE interference
#if !WITH_EDITOR
	FString ReportClient = FPaths::ConvertRelativePathToFull(FPlatformProcess::GenerateApplicationPath(TEXT("CrashReportClient"), EBuildConfiguration::Development));
	char CrashReportClient[PATH_MAX+1];
	FCStringAnsi::Strncpy(CrashReportClient, TCHAR_TO_UTF8(*ReportClient), PATH_MAX + 1);
	struct stat Stat;
	stat(CrashReportClient, &Stat);
	bEnableCustomSignalHandling = (S_ISREG(Stat.st_mode)) && ((Stat.st_mode & S_IXUSR) != 0);	// CRC exists and executable
#endif

	if (bEnableCustomSignalHandling && !FMacApplicationInfo::CrashReporter && !GCrashMalloc)
	{
		FPlatformProcess::UserHomeDir(); // This may be called from the crash handler, so we call it here to initialize it early.
										 // This way the crash handler will use a cached string and we will limit allocations and avoid creating an autorelease pool.

		// Configure the crash handler malloc zone to reserve some VM space for itself
		GCrashMalloc = new FMacMallocCrashHandler( 128 * 1024 * 1024 );
		
	#if defined(USE_UNTESTED_PL_CRASHREPORTER)
		// #agrant todo - Needs examined
		PLCrashReporterConfig* Config = [[[PLCrashReporterConfig alloc] initWithSignalHandlerType: PLCrashReporterSignalHandlerTypeBSD
																		symbolicationStrategy: PLCrashReporterSymbolicationStrategyNone
																		] autorelease];
	#else
		PLCrashReporterConfig* Config = [[[PLCrashReporterConfig alloc] initWithSignalHandlerType: PLCrashReporterSignalHandlerTypeBSD
																		symbolicationStrategy: PLCrashReporterSymbolicationStrategyNone
																		crashReportFolder:FMacApplicationInfo::TemporaryCrashReportFolder().GetNSString()
																		crashReportName:FMacApplicationInfo::TemporaryCrashReportName().GetNSString()] autorelease];
	#endif

		FMacApplicationInfo::CrashReporter = [[PLCrashReporter alloc] initWithConfiguration: Config];
		
		PLCrashReporterCallbacks CrashReportCallback = {
			.version = 0,
			.context = nullptr,
			.handleSignal = PLCrashReporterHandler
		};
		
		[FMacApplicationInfo::CrashReporter setCrashCallbacks: &CrashReportCallback];

		NSError* Error = nil;
		if ([FMacApplicationInfo::CrashReporter enableCrashReporterAndReturnError: &Error])
		{
			/* GMacStackIgnoreDepth = 0; */
		}
		else
		{
			UE_LOGF(LogMac, Log,  "Failed to enable PLCrashReporter: %ls", *FString([Error localizedDescription]) );
			
			UE_LOGF(LogMac, Log,  "Falling back to native signal handlers.");
			
			struct sigaction Action;
			FMemory::Memzero(&Action, sizeof(struct sigaction));
			Action.sa_sigaction = PlatformCrashHandler;
			sigemptyset(&Action.sa_mask);
			Action.sa_flags = SA_SIGINFO | SA_RESTART | SA_ONSTACK;
			sigaction(SIGQUIT, &Action, NULL);	// SIGQUIT is a user-initiated "crash".
			sigaction(SIGILL, &Action, NULL);
			sigaction(SIGEMT, &Action, NULL);
			sigaction(SIGFPE, &Action, NULL);
			sigaction(SIGBUS, &Action, NULL);
			sigaction(SIGSEGV, &Action, NULL);
			sigaction(SIGSYS, &Action, NULL);
			sigaction(SIGABRT, &Action, NULL);
		}
	}
}

FMacCrashContext::FMacCrashContext(ECrashContextType InType, const TCHAR* InErrorMessage)
	: FApplePlatformCrashContext(InType, InErrorMessage)
{
}

void FMacCrashContext::CopyMinidump(char const* OutputPath, char const* InputPath) const
{
	int ReportFile = open(OutputPath, O_CREAT|O_WRONLY, 0766);
	int DumpFile = open(InputPath, O_RDONLY, 0766);
	if (ReportFile != -1 && DumpFile != -1)
	{
		char Data[PATH_MAX];
		
		int Bytes = 0;
		while((Bytes = read(DumpFile, Data, PATH_MAX)) > 0)
		{
			write(ReportFile, Data, Bytes);
		}
		
		close(DumpFile);
		close(ReportFile);
		
		unlink(InputPath);
	}
}

void FMacCrashContext::GenerateInfoInFolder(char const* const InfoFolder) const
{
	// create a crash-specific directory
	char CrashInfoFolder[PATH_MAX] = {};
	FCStringAnsi::Strncpy(CrashInfoFolder, InfoFolder, PATH_MAX);
	
	if(!mkdir(CrashInfoFolder, 0766))
	{
		char FilePath[PATH_MAX] = {};
		
		// generate additional files from providers.
		FAdditionalCrashContextParams Params =
		{
			.CrashedThreadId = GetCrashedThreadId()
		};
		DumpAdditionalContext(*FString(CrashInfoFolder), Params);
		
		// generate "minidump" (Apple crash log format)
		FCStringAnsi::Strncpy(FilePath, CrashInfoFolder, PATH_MAX);
		FCStringAnsi::StrncatTruncateDest(FilePath, PATH_MAX, "/minidump.dmp");
		ConvertMinidump(FilePath, GMacAppInfo.PLCrashReportPath);
		
		// generate "info.txt" custom data for our server
		FCStringAnsi::Strncpy(FilePath, CrashInfoFolder, PATH_MAX);
		FCStringAnsi::StrncatTruncateDest(FilePath, PATH_MAX, "/info.txt");
		int ReportFile = open(FilePath, O_CREAT|O_WRONLY, 0766);
		if (ReportFile != -1)
		{
			WriteUTF16String(ReportFile, TEXT("GameName UE-"));
			WriteLine(ReportFile, *GMacAppInfo.AppName);
			
			WriteUTF16String(ReportFile, TEXT("BuildVersion 1.0."));
			WriteUTF16String(ReportFile, ItoTCHAR(FEngineVersion::Current().GetChangelist() >> 16, 10));
			WriteUTF16String(ReportFile, TEXT("."));
			WriteLine(ReportFile, ItoTCHAR(FEngineVersion::Current().GetChangelist() & 0xffff, 10));
			
			WriteUTF16String(ReportFile, TEXT("CommandLine "));
			WriteLine(ReportFile, *GMacAppInfo.CommandLine);
			
			WriteUTF16String(ReportFile, TEXT("BaseDir "));
			WriteLine(ReportFile, *GMacAppInfo.BranchBaseDir);
			
			WriteUTF16String(ReportFile, TEXT("MachineGuid "));
			WriteLine(ReportFile, *GMacAppInfo.MachineUUID);
			
			close(ReportFile);
		}
		
		// Introduces a new runtime crash context. Will replace all Windows related crash reporting.
		FCStringAnsi::Strncpy(FilePath, CrashInfoFolder, PATH_MAX);
		FCStringAnsi::StrncatTruncateDest(FilePath, PATH_MAX, "/" );
		FCStringAnsi::StrncatTruncateDest(FilePath, PATH_MAX, FGenericCrashContext::CrashContextRuntimeXMLNameA );
		SerializeAsXML( *FString(FilePath) );
		
		// copy log
		FCStringAnsi::Strncpy(FilePath, CrashInfoFolder, PATH_MAX);
		FCStringAnsi::StrncatTruncateDest(FilePath, PATH_MAX, "/");
		FCStringAnsi::StrncatTruncateDest(FilePath, PATH_MAX, (!GMacAppInfo.AppName.IsEmpty() ? GMacAppInfo.AppNameUTF8 : "Unreal"));
		FCStringAnsi::StrncatTruncateDest(FilePath, PATH_MAX, ".log");
		int LogSrc = open(GMacAppInfo.AppLogPath, O_RDONLY);
		int LogDst = open(FilePath, O_CREAT|O_WRONLY, 0766);
		
		char Data[PATH_MAX] = {};
		int Bytes = 0;
		while((Bytes = read(LogSrc, Data, PATH_MAX)) > 0)
		{
			write(LogDst, Data, Bytes);
		}
		
		// If present, include the crash report config file to pass config values to the CRC
		FCStringAnsi::Strncpy(FilePath, CrashInfoFolder, PATH_MAX);
		FCStringAnsi::StrncatTruncateDest(FilePath, PATH_MAX, "/");
		FCStringAnsi::StrncatTruncateDest(FilePath, PATH_MAX, FGenericCrashContext::CrashConfigFileNameA);
		int ConfigSrc = open(TCHAR_TO_ANSI(GetCrashConfigFilePath()), O_RDONLY);
		int ConfigDst = open(FilePath, O_CREAT | O_WRONLY, 0766);

		while ((Bytes = read(ConfigSrc, Data, PATH_MAX)) > 0)
		{
			write(ConfigDst, Data, Bytes);
		}

		close(ConfigDst);
		close(ConfigSrc);

		// Copy all the GPU restart logs from the user machine into our log
		if ( !GMacAppInfo.bIsSandboxed && GIsGPUCrashed && GMacAppInfo.KernelErrorDir )
		{
			struct dirent DirEntry;
			struct dirent* DirResult;
			while(readdir_r(GMacAppInfo.KernelErrorDir, &DirEntry, &DirResult) == 0 && DirResult == &DirEntry)
			{
				if (strstr(DirEntry.d_name, ".gpuRestart"))
				{
					FCStringAnsi::Strncpy(FilePath, "/Library/Logs/DiagnosticReports/", PATH_MAX);
					FCStringAnsi::StrncatTruncateDest(FilePath, PATH_MAX, DirEntry.d_name);
					if (access(FilePath, R_OK|F_OK) == 0)
					{
						char const* SysLogHeader = "\nAppending GPU Restart Log: ";
						write(LogDst, SysLogHeader, strlen(SysLogHeader));
						
						write(LogDst, FilePath, strlen(FilePath));
						write(LogDst, "\n", strlen("\n"));

						int SysLogSrc = open(FilePath, O_RDONLY);
						while((Bytes = read(SysLogSrc, Data, PATH_MAX)) > 0)
						{
							write(LogDst, Data, Bytes);
						}
						close(SysLogSrc);
					}
				}
			}
		}
		
		// Copy the system log to capture GPU restarts and other nasties not reported by our application
		if ( !GMacAppInfo.bIsSandboxed && GMacAppInfo.SystemLogSize >= 0 && access("/var/log/system.log", R_OK|F_OK) == 0 )
		{
			char const* SysLogHeader = "\nAppending System Log:\n";
			write(LogDst, SysLogHeader, strlen(SysLogHeader));
			
			int SysLogSrc = open("/var/log/system.log", O_RDONLY);
			
			// Attempt to capture only the system log from while our application was running but
			if (lseek(SysLogSrc, GMacAppInfo.SystemLogSize, SEEK_SET) != GMacAppInfo.SystemLogSize)
			{
				close(SysLogSrc);
				SysLogSrc = open("/var/log/system.log", O_RDONLY);
			}
			
			while((Bytes = read(SysLogSrc, Data, PATH_MAX)) > 0)
			{
				write(LogDst, Data, Bytes);
			}
			close(SysLogSrc);
		}
		
		close(LogDst);
		close(LogSrc);
		// best effort, so don't care about result: couldn't copy -> tough, no log
		
		// copy crash video if there is one
		if ( access(GMacAppInfo.CrashReportVideo, R_OK|F_OK) == 0 )
		{
			FCStringAnsi::Strncpy(FilePath, CrashInfoFolder, PATH_MAX);
			FCStringAnsi::StrncatTruncateDest(FilePath, PATH_MAX, "/");
			FCStringAnsi::StrncatTruncateDest(FilePath, PATH_MAX, "CrashVideo.avi");
			int VideoSrc = open(GMacAppInfo.CrashReportVideo, O_RDONLY);
			int VideoDst = open(FilePath, O_CREAT|O_WRONLY, 0766);
			
			while((Bytes = read(VideoSrc, Data, PATH_MAX)) > 0)
			{
				write(VideoDst, Data, Bytes);
			}
			close(VideoDst);
			close(VideoSrc);
		}
	}
}

void FMacCrashContext::GenerateInfoAndLaunchCrashReporter(bool IsCrash /* false means ensure */) const
{
	// Prevent CrashReportClient from spawning another CrashReportClient.
	bool bCanRunCrashReportClient = FCString::Stristr( *(GMacAppInfo.ExecutableName), TEXT( "CrashReportClient" ) ) == nullptr;

	bool bImplicitSend = false;
	if (!UE_EDITOR && GConfig)
	{
		// Only check if we are in a non-editor build
		GConfig->GetBool(TEXT("CrashReportClient"), TEXT("bImplicitSend"), bImplicitSend, GEngineIni);
	}

	bool bSendUnattendedBugReports = true;
	if (GConfig)
	{
		GConfig->GetBool(TEXT("/Script/UnrealEd.CrashReportsPrivacySettings"), TEXT("bSendUnattendedBugReports"), bSendUnattendedBugReports, GEditorSettingsIni);
	}

	bool bSendUsageData = true;
	if (GConfig)
	{
		GConfig->GetBool(TEXT("/Script/UnrealEd.AnalyticsPrivacySettings"), TEXT("bSendUsageData"), bSendUsageData, GEditorSettingsIni);
	}

	// NOTE: A blueprint-only game packaged from a vanilla engine downloaded from Epic Game Store isn't considered a 'Licensee' version because the engine was built by Epic.
	//       There is no way at the moment do distinguish this case properly.
	if (BuildSettings::IsLicenseeVersion() && !UE_EDITOR)
	{
		// do not send unattended reports in licensees' builds except for the editor, where it is governed by the above setting
		bSendUnattendedBugReports = false;
		bSendUsageData = false;
	}

	// A non-interactive ensure is unattended: gate it on the Bug Reports opt-out, matching Windows/Unix.
	const bool bUnattended = GMacAppInfo.bIsUnattended || (!IsCrash && !IsInteractiveEnsureMode()) || IsRunningDedicatedServer();
	if (bUnattended && !bSendUnattendedBugReports)
	{
		bCanRunCrashReportClient = false;
	}

	if(bCanRunCrashReportClient)
	{
		// create a crash-specific directory
		FString CrashInfoFolder = FString::Printf(TEXT("%s/%s-UE-%s-pid-%d-%s"), UTF8_TO_TCHAR(GMacAppInfo.CrashReportPath), IsCrash ? TEXT("CrashReport") : TEXT("EnsureReport"), UTF8_TO_TCHAR(GMacAppInfo.AppNameUTF8), (int32)getpid(), *GMacAppInfo.RunUUID.ToString());

		// Do not inline this! The lifetime of this object needs to extend over the usage of Argv in posix_spawn() call below.
		auto CrashInfoFolderUTF8 = TStringConversion<FTCHARToUTF8_Convert>(*CrashInfoFolder);
		
		CaptureAllThreadContext(0);
		GenerateInfoInFolder(CrashInfoFolderUTF8.Get());

		CrashInfoFolder += TEXT("/");

		pid_t		CrcPID;
		int32		Status		= -1;
		const char *Argv[16]	= { NULL };
		int32		Argc		= 0;

		Argv[Argc++] = "CrashReportClient";
		Argv[Argc++] = CrashInfoFolderUTF8.Get();

		if (IsCrash)
		{
			if (bImplicitSend)
			{
				Argv[Argc++] = "-Unattended";
				Argv[Argc++] = "-ImplicitSend";
			}
			else if(GMacAppInfo.bIsUnattended)
			{
				Argv[Argc++] ="-Unattended";
			}
			else
			{
				if (!bSendUsageData)
				{
					Argv[Argc++] = "-NoAnalytics";
				}
			}
		}
		else	/* ensure */
		{
			if (!IsInteractiveEnsureMode())
			{
				Argv[Argc++] = "-Unattended";
			}
			if (!bSendUsageData)
			{
				Argv[Argc++] = "-NoAnalytics";
			}
		}

		// We've generated the crash report above since we were able to,
		// now we need to make sure CRC actually exists and is executable before
		// trying to run it.
		struct stat Stat;
		stat(GMacAppInfo.CrashReportClient, &Stat);
		if (!S_ISREG(Stat.st_mode))
		{
			UE_LOGF(LogMac, Error, "Unable to locate CrashReporterClient: %ls", UTF8_TO_TCHAR(GMacAppInfo.CrashReportClient));
		}
		else if ((Stat.st_mode & S_IXUSR) == 0)
		{
			UE_LOGF(LogMac, Error, "Unable to execute CrashReporterClient, please run: chmod u+x %ls", UTF8_TO_TCHAR(GMacAppInfo.CrashReportClient));
		}
		else
		{
			posix_spawn_file_actions_t FileActions;
			posix_spawn_file_actions_init(&FileActions);

			posix_spawnattr_t SpawnAttr;
			posix_spawnattr_init(&SpawnAttr);

			{
				uint16 SpawnFlags = POSIX_SPAWN_SETPGROUP;
				posix_spawnattr_setflags(&SpawnAttr, SpawnFlags);
			}

			extern char **environ; // provided by libc

			// Use posix_spawn() as it is async-signal safe, CreateProc can fail in Cocoa.
			Status = posix_spawn(&CrcPID, GMacAppInfo.CrashReportClient, &FileActions, &SpawnAttr, (char *const *)Argv, environ);

			posix_spawn_file_actions_destroy(&FileActions);
			posix_spawnattr_destroy(&SpawnAttr);

			if (Status != 0)
			{
				UE_LOGF(LogMac, Error, "FMacPlatformMisc::GenerateCrashInfoAndLaunchReporter: posix_spawn() failed (%d, %ls)", Status, UTF8_TO_TCHAR(strerror(Status)));
			}
		}
	}

	if(IsCrash)
	{
		// Sandboxed applications re-raise the signal to trampoline into the system crash reporter as suppressing it may fall foul of Apple's Mac App Store rules.
		// @todo Submit an application to the MAS & see whether Apple's reviewers highlight our crash reporting or trampolining to the system reporter.
		if (GMacAppInfo.bIsSandboxed)
		{
			struct sigaction Action;
			FMemory::Memzero(&Action, sizeof(struct sigaction));
			Action.sa_handler = SIG_DFL;
			sigemptyset(&Action.sa_mask);
			sigaction(SIGQUIT, &Action, NULL);
			sigaction(SIGILL, &Action, NULL);
			sigaction(SIGEMT, &Action, NULL);
			sigaction(SIGFPE, &Action, NULL);
			sigaction(SIGBUS, &Action, NULL);
			sigaction(SIGSEGV, &Action, NULL);
			sigaction(SIGSYS, &Action, NULL);
			sigaction(SIGABRT, &Action, NULL);
			sigaction(SIGTRAP, &Action, NULL);
			
			raise(Signal);
		}
		
		_Exit(1);
	} 
}

void FMacCrashContext::GenerateCrashInfoAndLaunchReporter() const
{
	GenerateInfoAndLaunchCrashReporter(true);
}

void FMacCrashContext::GenerateEnsureInfoAndLaunchReporter() const
{
	GenerateInfoAndLaunchCrashReporter(false);
}

static FString AllThreadContexts;

void FMacCrashContext::AddThreadContext(
	uint32 ThreadIdEnteredOn,
	uint32 ThreadId,
	const FString& ThreadName,
	const TArray<FCrashStackFrame>& PortableCallStack) const
{
	AllThreadContexts += TEXT("<Thread>");
	{
		AllThreadContexts += TEXT("<CallStack>");

		int32 MaxModuleNameLen = 0;
		for (const FCrashStackFrame& StFrame : PortableCallStack)
		{
			MaxModuleNameLen = FMath::Max(MaxModuleNameLen, StFrame.ModuleName.Len());
		}

		FString CallstackStr;
		for (const FCrashStackFrame& StFrame : PortableCallStack)
		{
			CallstackStr += FString::Printf(TEXT("%-*s 0x%016llx + %-16llx"), MaxModuleNameLen + 1, *StFrame.ModuleName, StFrame.BaseAddress, StFrame.Offset);
			CallstackStr += LINE_TERMINATOR;
		}
		AppendEscapedXMLString(AllThreadContexts, *CallstackStr);
		AllThreadContexts += TEXT("</CallStack>") LINE_TERMINATOR;
	}

	AllThreadContexts += FString::Printf(TEXT("<IsCrashed>%s</IsCrashed>") LINE_TERMINATOR, (ThreadId == ThreadIdEnteredOn) ? TEXT("true") : TEXT("false"));
	// TODO: do we need thread register states?
	AllThreadContexts += TEXT("<Registers></Registers>") LINE_TERMINATOR;
	AllThreadContexts += FString::Printf(TEXT("<ThreadID>%d</ThreadID>") LINE_TERMINATOR, ThreadId);
	AllThreadContexts += FString::Printf(TEXT("<ThreadName>%s</ThreadName>") LINE_TERMINATOR, *ThreadName);
	AllThreadContexts += TEXT("</Thread>");
	AllThreadContexts += LINE_TERMINATOR;
}

void FMacCrashContext::CaptureAllThreadContext(uint32 ThreadIdEnteredOn) const
{
	TArray<typename FThreadManager::FThreadStackBackTrace> StackTraces;
	FThreadManager::Get().GetAllThreadStackBackTraces(StackTraces);

	for (int32 Idx = 0; Idx < StackTraces.Num(); ++Idx)
	{
		const FThreadManager::FThreadStackBackTrace& ThreadStTrace = StackTraces[Idx];
		const uint32 ThreadId     = ThreadStTrace.ThreadId;
		const FString& ThreadName = ThreadStTrace.ThreadName;

		if (Idx == 0)
		{
			// current thread, which is crashing
			AddThreadContext(ThreadId, ThreadId, ThreadName, CallStack);
		}
		else
		{
			TArray<FCrashStackFrame> PortableStack;
			GetPortableCallStack(ThreadStTrace.ProgramCounters.GetData(), ThreadStTrace.ProgramCounters.Num(), PortableStack);

			AddThreadContext(ThreadIdEnteredOn, ThreadId, ThreadName, PortableStack);
		}
	}
}

bool FMacCrashContext::GetPlatformAllThreadContextsString(FString& OutStr) const
{
	OutStr = AllThreadContexts;
	AllThreadContexts = TEXT("");
	return !OutStr.IsEmpty();
}

void ReportAssert(const TCHAR* ErrorMessage, void* ErrorProgramCounter)
{
	GCrashErrorMessage = ErrorMessage;
	GCrashErrorProgramCounter = ErrorProgramCounter;
	GCrashErrorType = ECrashContextType::Assert;
	FPlatformMisc::RaiseException(1);
}

void ReportGPUCrash(const TCHAR* ErrorMessage, void* ErrorProgramCounter)
{
	if (ErrorProgramCounter == nullptr)
	{
		ErrorProgramCounter = PLATFORM_RETURN_ADDRESS();
	}

	GCrashErrorMessage = ErrorMessage;
	GCrashErrorProgramCounter = ErrorProgramCounter;
	GCrashErrorType = ECrashContextType::GPUCrash;
	FPlatformMisc::RaiseException(1);
}

static FCriticalSection EnsureLock;
static bool bReentranceGuard = false;

void ReportEnsure( const TCHAR* ErrorMessage, void* ErrorProgramCounter )
{
	// Simple re-entrance guard.
	EnsureLock.Lock();
	
	if( bReentranceGuard )
	{
		EnsureLock.Unlock();
		return;
	}
	
	bReentranceGuard = true;
	
	if(FMacApplicationInfo::CrashReporter != nil)
	{
		siginfo_t Signal;
		Signal.si_signo = SIGTRAP;
		Signal.si_code = TRAP_TRACE;
		Signal.si_addr = ErrorProgramCounter;
		
		FMacCrashContext EnsureContext(ECrashContextType::Ensure, ErrorMessage);
		EnsureContext.InitFromSignal(SIGTRAP, &Signal, nullptr);
		EnsureContext.ErrorFrame = ErrorProgramCounter;
		EnsureContext.GenerateEnsureInfoAndLaunchReporter();
	}
	
	bReentranceGuard = false;
	EnsureLock.Unlock();
}

void ReportHang(const TCHAR* ErrorMessage, const uint64* StackFrames, int32 NumStackFrames, uint32 HungThreadId)
{
	EnsureLock.Lock();
	if (!bReentranceGuard && FMacApplicationInfo::CrashReporter != nil)
	{
		bReentranceGuard = true;

		FMacCrashContext EnsureContext(ECrashContextType::Hang, ErrorMessage);
		EnsureContext.SetPortableCallStack(StackFrames, NumStackFrames);

		if (CVarMacPlatformDumpAllThreadsOnHang.AsVariable()->GetInt())
		{
			EnsureContext.CaptureAllThreadContext(HungThreadId);
		}

		EnsureContext.GenerateEnsureInfoAndLaunchReporter();

		bReentranceGuard = false;
	}
	EnsureLock.Unlock();
}

typedef NSArray* (*MTLCopyAllDevices)(void);

bool FMacPlatformMisc::HasPlatformFeature(const TCHAR* FeatureName)
{
	if (FCString::Stricmp(FeatureName, TEXT("Metal")) == 0)
	{
#if PLATFORM_MAC_ARM64
        return true;
#else
        static bool bHasCheckedForMetal = false;
		static bool bHasMetal = false;
        
		if (!bHasCheckedForMetal)
        {
            bHasCheckedForMetal = true;
            if (FModuleManager::Get().ModuleExists(TEXT("MetalRHI")))
            {
                // Find out if there are any Metal devices on the system - some Mac's have none
                void* DLLHandle = FPlatformProcess::GetDllHandle(TEXT("/System/Library/Frameworks/Metal.framework/Metal"));
                if (DLLHandle)
                {
                    TArray<FMacGPUDescriptor> const& GPUs = FPlatformMisc::GetGPUDescriptors();
                    
                    if (GPUs.Num() > 0)
                    {
                        for (FMacGPUDescriptor const& GPU : GPUs)
                        {
                            NSString* GPUName = [GPU.GPUName length] > 0 ? GPU.GPUName : @"Unnamed GPU";
                            NSString* GPUMetalBundle = [GPU.GPUMetalBundle length] > 0 ? GPU.GPUMetalBundle : @"null";
                            
                            UE_LOGF(LogMac, Display, "Checking GPU: %ls (MetalBundle: %ls)", *FString(GPUName), *FString(GPUMetalBundle));

                            if ([GPU.GPUMetalBundle length] > 0)
                            {
                                bHasMetal = true;
                                break;
                            }
                        }
                    }
                    else
                    {
                        UE_LOGF(LogMac, Error, "No GPUs found!");
                    }
                    
                    FPlatformProcess::FreeDllHandle(DLLHandle);
                }
                else
                {
                    UE_LOGF(LogMac, Error, "Could not get handle to Metal.Framework");
                }
            }
            else
            {
                UE_LOGF(LogMac, Error, "No MetalRHI Module");
            }
        }
        
        return bHasMetal;
#endif
	}

	return FGenericPlatformMisc::HasPlatformFeature(FeatureName);
}

/*------------------------------------------------------------------------------
 DriverMonitor - Stats groups for Mac Driver Monitor performance statistics available from IOKit & Driver Monitor, so that they may be logged within our performance capture tools.
    These stats provide a lot of information about what the driver is doing at any point and being able to see where the time & memory is going can be very helpful when debugging.
    They would be especially helpful if they could be logged over time alongside out own RHI stats to compare what we *think* we are doing with what is *actually* happening.
 ------------------------------------------------------------------------------*/

DECLARE_STATS_GROUP(TEXT("Driver Monitor"),STATGROUP_DriverMonitor, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("Driver Monitor (AMD specific)"),STATGROUP_DriverMonitorAMD, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("Driver Monitor (Intel specific)"),STATGROUP_DriverMonitorIntel, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("Driver Monitor (Nvidia specific)"),STATGROUP_DriverMonitorNvidia, STATCAT_Advanced);

DECLARE_FLOAT_COUNTER_STAT(TEXT("Device Utilization %"),STAT_DriverMonitorDeviceUtilisation,STATGROUP_DriverMonitor);
DECLARE_FLOAT_COUNTER_STAT(TEXT("Device Utilization % at cur p-state"),STAT_DM_I_DeviceUtilisationAtPState,STATGROUP_DriverMonitorIntel);
DECLARE_FLOAT_COUNTER_STAT(TEXT("Device Unit 0 Utilization %"),STAT_DM_I_Device0Utilisation,STATGROUP_DriverMonitorIntel);
DECLARE_FLOAT_COUNTER_STAT(TEXT("Device Unit 1 Utilization %"),STAT_DM_I_Device1Utilisation,STATGROUP_DriverMonitorIntel);
DECLARE_FLOAT_COUNTER_STAT(TEXT("Device Unit 2 Utilization %"),STAT_DM_I_Device2Utilisation,STATGROUP_DriverMonitorIntel);
DECLARE_FLOAT_COUNTER_STAT(TEXT("Device Unit 3 Utilization %"),STAT_DM_I_Device3Utilisation,STATGROUP_DriverMonitorIntel);

DECLARE_MEMORY_STAT(TEXT("VRAM Used Bytes"),STAT_DriverMonitorVRAMUsedBytes,STATGROUP_DriverMonitor);
DECLARE_MEMORY_STAT(TEXT("VRAM Free Bytes"),STAT_DriverMonitorVRAMFreeBytes,STATGROUP_DriverMonitor);
DECLARE_MEMORY_STAT(TEXT("VRAM Largest Free Bytes"),STAT_DriverMonitorVRAMLargestFreeBytes,STATGROUP_DriverMonitor);
DECLARE_MEMORY_STAT(TEXT("In Use Vid Mem Bytes"),STAT_DriverMonitorInUseVidMemBytes,STATGROUP_DriverMonitor);
DECLARE_MEMORY_STAT(TEXT("In Use Sys Mem Bytes"),STAT_DriverMonitorInUseSysMemBytes,STATGROUP_DriverMonitor);

DECLARE_MEMORY_STAT(TEXT("DMA Used Bytes"),STAT_DriverMonitorgartUsedBytes,STATGROUP_DriverMonitor);
DECLARE_MEMORY_STAT(TEXT("DMA Free Bytes"),STAT_DriverMonitorgartFreeBytes,STATGROUP_DriverMonitor);
DECLARE_MEMORY_STAT(TEXT("DMA Bytes"),STAT_DriverMonitorgartSizeBytes,STATGROUP_DriverMonitor);
DECLARE_MEMORY_STAT(TEXT("DMA Data Mapped"),STAT_DriverMonitorgartMapInBytesPerSample,STATGROUP_DriverMonitor);
DECLARE_MEMORY_STAT(TEXT("DMA Data Unmapped"),STAT_DriverMonitorgartMapOutBytesPerSample,STATGROUP_DriverMonitor);

DECLARE_MEMORY_STAT(TEXT("Texture Page-off Bytes"),STAT_DriverMonitortexturePageOutBytes,STATGROUP_DriverMonitor);
DECLARE_MEMORY_STAT(TEXT("Texture Read-off Bytes"),STAT_DriverMonitortextureReadOutBytes,STATGROUP_DriverMonitor);
DECLARE_MEMORY_STAT(TEXT("Texture Volunteer Unload Bytes"),STAT_DriverMonitortextureVolunteerUnloadBytes,STATGROUP_DriverMonitor);
DECLARE_MEMORY_STAT(TEXT("AGP Texture Creation Bytes"),STAT_DriverMonitoragpTextureCreationBytes,STATGROUP_DriverMonitor);
DECLARE_DWORD_COUNTER_STAT(TEXT("AGP Texture Creation Count"),STAT_DriverMonitoragpTextureCreationCount,STATGROUP_DriverMonitor);
DECLARE_MEMORY_STAT(TEXT("AGP Ref Texture Creation Bytes"),STAT_DriverMonitoragprefTextureCreationBytes,STATGROUP_DriverMonitor);
DECLARE_DWORD_COUNTER_STAT(TEXT("AGP Ref Texture Creation Count"),STAT_DriverMonitoragprefTextureCreationCount,STATGROUP_DriverMonitor);

DECLARE_MEMORY_STAT(TEXT("IOSurface Page-In Bytes"),STAT_DriverMonitorioSurfacePageInBytes,STATGROUP_DriverMonitor);
DECLARE_MEMORY_STAT(TEXT("IOSurface Page-Out Bytes"),STAT_DriverMonitorioSurfacePageOutBytes,STATGROUP_DriverMonitor);
DECLARE_MEMORY_STAT(TEXT("IOSurface Read-Out Bytes"),STAT_DriverMonitorioSurfaceReadOutBytes,STATGROUP_DriverMonitor);
DECLARE_DWORD_COUNTER_STAT(TEXT("IOSurface Texture Creation Count"),STAT_DriverMonitoriosurfaceTextureCreationCount,STATGROUP_DriverMonitor);
DECLARE_MEMORY_STAT(TEXT("IOSurface Texture Creation Bytes"),STAT_DriverMonitoriosurfaceTextureCreationBytes,STATGROUP_DriverMonitor);

DECLARE_MEMORY_STAT(TEXT("OOL Texture Page-In Bytes"),STAT_DriverMonitoroolTexturePageInBytes,STATGROUP_DriverMonitor);
DECLARE_DWORD_COUNTER_STAT(TEXT("OOL Texture Creation Count"),STAT_DriverMonitoroolTextureCreationCount,STATGROUP_DriverMonitor);
DECLARE_MEMORY_STAT(TEXT("OOL Texture Creation Bytes"),STAT_DriverMonitoroolTextureCreationBytes,STATGROUP_DriverMonitor);

DECLARE_MEMORY_STAT(TEXT("orphanedNonReusableSysMemoryBytes"),STAT_DriverMonitororphanedNonReusableSysMemoryBytes, STATGROUP_DriverMonitor);
DECLARE_DWORD_COUNTER_STAT(TEXT("orphanedNonReusableSysMemoryCount"),STAT_DriverMonitororphanedNonReusableSysMemoryCount, STATGROUP_DriverMonitor);
DECLARE_MEMORY_STAT(TEXT("orphanedReusableSysMemoryBytes"),STAT_DriverMonitororphanedReusableSysMemoryBytes, STATGROUP_DriverMonitor);
DECLARE_DWORD_COUNTER_STAT(TEXT("orphanedReusableSysMemoryCount"),STAT_DriverMonitororphanedReusableSysMemoryCount, STATGROUP_DriverMonitor);
DECLARE_FLOAT_COUNTER_STAT(TEXT("orphanedReusableSysMemoryHitRate"),STAT_DriverMonitororphanedReusableSysMemoryHitRate, STATGROUP_DriverMonitor);
DECLARE_MEMORY_STAT(TEXT("orphanedNonReusableVidMemoryBytes"),STAT_DriverMonitororphanedNonReusableVidMemoryBytes, STATGROUP_DriverMonitor);
DECLARE_DWORD_COUNTER_STAT(TEXT("orphanedNonReusableVidMemoryCount"),STAT_DriverMonitororphanedNonReusableVidMemoryCount, STATGROUP_DriverMonitor);
DECLARE_MEMORY_STAT(TEXT("orphanedReusableVidMemoryBytes"),STAT_DriverMonitororphanedReusableVidMemoryBytes, STATGROUP_DriverMonitor);
DECLARE_DWORD_COUNTER_STAT(TEXT("orphanedReusableVidMemoryCount"),STAT_DriverMonitororphanedReusableVidMemoryCount, STATGROUP_DriverMonitor);
DECLARE_FLOAT_COUNTER_STAT(TEXT("orphanedReusableVidMemoryHitRate"),STAT_DriverMonitororphanedReusableVidMemoryHitRate, STATGROUP_DriverMonitor);

DECLARE_MEMORY_STAT(TEXT("stdTextureCreationBytes"),STAT_DriverMonitorstdTextureCreationBytes, STATGROUP_DriverMonitor);
DECLARE_DWORD_COUNTER_STAT(TEXT("stdTextureCreationCount"),STAT_DriverMonitorstdTextureCreationCount, STATGROUP_DriverMonitor);
DECLARE_MEMORY_STAT(TEXT("stdTexturePageInBytes"),STAT_DriverMonitorstdTexturePageInBytes, STATGROUP_DriverMonitor);
DECLARE_MEMORY_STAT(TEXT("surfaceBufferPageInBytes"),STAT_DriverMonitorsurfaceBufferPageInBytes, STATGROUP_DriverMonitor);
DECLARE_MEMORY_STAT(TEXT("surfaceBufferPageOutBytes"),STAT_DriverMonitorsurfaceBufferPageOutBytes, STATGROUP_DriverMonitor);
DECLARE_MEMORY_STAT(TEXT("surfaceBufferReadOutBytes"),STAT_DriverMonitorsurfaceBufferReadOutBytes, STATGROUP_DriverMonitor);
DECLARE_DWORD_COUNTER_STAT(TEXT("surfaceTextureCreationCount"),STAT_DriverMonitorsurfaceTextureCreationCount, STATGROUP_DriverMonitor);

DECLARE_CYCLE_STAT(TEXT("CPU Wait For GPU"),STAT_DriverMonitorCPUWaitForGPU,STATGROUP_DriverMonitor);
DECLARE_CYCLE_STAT(TEXT("CPU Wait to Submit Commands"),STAT_DriverMonitorCPUWaitToSubmit,STATGROUP_DriverMonitor);
DECLARE_CYCLE_STAT(TEXT("CPU Wait to perform Surface Read"),STAT_DriverMonitorCPUWaitToSurfaceRead,STATGROUP_DriverMonitor);
DECLARE_CYCLE_STAT(TEXT("CPU Wait to perform Surface Resize"),STAT_DriverMonitorCPUWaitToSurfaceResize,STATGROUP_DriverMonitor);
DECLARE_CYCLE_STAT(TEXT("CPU Wait to perform Surface Write"),STAT_DriverMonitorCPUWaitToSurfaceWrite,STATGROUP_DriverMonitor);
DECLARE_CYCLE_STAT(TEXT("CPU Wait to perform VRAM Surface page-off"),STAT_DriverMonitorCPUWaitToSurfacePageOff,STATGROUP_DriverMonitor);
DECLARE_CYCLE_STAT(TEXT("CPU Wait to perform VRAM Surface page-on"),STAT_DriverMonitorCPUWaitToSurfacePageOn,STATGROUP_DriverMonitor);
DECLARE_CYCLE_STAT(TEXT("CPU Wait to reclaim Surface GART Backing Store"),STAT_DriverMonitorCPUWaitToReclaimSurfaceGART,STATGROUP_DriverMonitor);
DECLARE_CYCLE_STAT(TEXT("CPU Wait to perform VRAM Eviction"),STAT_DriverMonitorCPUWaitToVRAMEvict,STATGROUP_DriverMonitor);
DECLARE_CYCLE_STAT(TEXT("CPU Wait to free Data Buffer"),STAT_DriverMonitorCPUWaitToFreeDataBuffer,STATGROUP_DriverMonitor);

DECLARE_DWORD_COUNTER_STAT(TEXT("surfaceCount"), STAT_DriverMonitorSurfaceCount, STATGROUP_DriverMonitor);
DECLARE_DWORD_COUNTER_STAT(TEXT("textureCount"), STAT_DriverMonitorTextureCount, STATGROUP_DriverMonitor);

DECLARE_FLOAT_COUNTER_STAT(TEXT("GPU Core Utilization"), STAT_DM_NV_GPUCoreUtilization, STATGROUP_DriverMonitorNvidia);
DECLARE_FLOAT_COUNTER_STAT(TEXT("GPU Memory Utilization"), STAT_DM_NV_GPUMemoryUtilization, STATGROUP_DriverMonitorNvidia);

DECLARE_DWORD_COUNTER_STAT(TEXT("HWChannel C0 | Commands Completed"), STAT_DM_AMD_HWChannelC0Complete, STATGROUP_DriverMonitorAMD);
DECLARE_DWORD_COUNTER_STAT(TEXT("HWChannel C0 | Commands Submitted"), STAT_DM_AMD_HWChannelC0Submit, STATGROUP_DriverMonitorAMD);
DECLARE_DWORD_COUNTER_STAT(TEXT("HWChannel C1 | Commands Completed"), STAT_DM_AMD_HWChannelC1Complete, STATGROUP_DriverMonitorAMD);
DECLARE_DWORD_COUNTER_STAT(TEXT("HWChannel C1 | Commands Submitted"), STAT_DM_AMD_HWChannelC1Submit, STATGROUP_DriverMonitorAMD);
DECLARE_DWORD_COUNTER_STAT(TEXT("HWChannel DMA0 | Commands Completed"), STAT_DM_AMD_HWChannelDMA0Complete, STATGROUP_DriverMonitorAMD);
DECLARE_DWORD_COUNTER_STAT(TEXT("HWChannel DMA0 | Commands Submitted"), STAT_DM_AMD_HWChannelDMA0Submit, STATGROUP_DriverMonitorAMD);
DECLARE_DWORD_COUNTER_STAT(TEXT("HWChannel DMA1 | Commands Completed"), STAT_DM_AMD_HWChannelDMA1Complete, STATGROUP_DriverMonitorAMD);
DECLARE_DWORD_COUNTER_STAT(TEXT("HWChannel DMA1 | Commands Submitted"), STAT_DM_AMD_HWChannelDMA1Submit, STATGROUP_DriverMonitorAMD);
DECLARE_DWORD_COUNTER_STAT(TEXT("HWChannel GFX | Commands Completed"), STAT_DM_AMD_HWChannelGFXComplete, STATGROUP_DriverMonitorAMD);
DECLARE_DWORD_COUNTER_STAT(TEXT("HWChannel GFX | Commands Submitted"), STAT_DM_AMD_HWChannelGFXSubmit, STATGROUP_DriverMonitorAMD);
DECLARE_DWORD_COUNTER_STAT(TEXT("HWChannel SPU | Commands Completed"), STAT_DM_AMD_HWChannelSPUComplete, STATGROUP_DriverMonitorAMD);
DECLARE_DWORD_COUNTER_STAT(TEXT("HWChannel SPU | Commands Submitted"), STAT_DM_AMD_HWChannelSPUSubmit, STATGROUP_DriverMonitorAMD);
DECLARE_DWORD_COUNTER_STAT(TEXT("HWChannel UVD | Commands Completed"), STAT_DM_AMD_HWChannelUVDComplete, STATGROUP_DriverMonitorAMD);
DECLARE_DWORD_COUNTER_STAT(TEXT("HWChannel UVD | Commands Submitted"), STAT_DM_AMD_HWChannelUVDSubmit, STATGROUP_DriverMonitorAMD);
DECLARE_DWORD_COUNTER_STAT(TEXT("HWChannel VCE | Commands Completed"), STAT_DM_AMD_HWChannelVCEComplete, STATGROUP_DriverMonitorAMD);
DECLARE_DWORD_COUNTER_STAT(TEXT("HWChannel VCE | Commands Submitted"), STAT_DM_AMD_HWChannelVCESubmit, STATGROUP_DriverMonitorAMD);
DECLARE_DWORD_COUNTER_STAT(TEXT("HWChannel VCELLQ | Commands Completed"), STAT_DM_AMD_HWChannelVCELLQComplete, STATGROUP_DriverMonitorAMD);
DECLARE_DWORD_COUNTER_STAT(TEXT("HWChannel VCELLQ | Commands Submitted"), STAT_DM_AMD_HWChannelVCELLQSubmit, STATGROUP_DriverMonitorAMD);
DECLARE_DWORD_COUNTER_STAT(TEXT("HWChannel KIQ | Commands Completed"), STAT_DM_AMD_HWChannelKIQComplete, STATGROUP_DriverMonitorAMD);
DECLARE_DWORD_COUNTER_STAT(TEXT("HWChannel KIQ | Commands Submitted"), STAT_DM_AMD_HWChannelKIQSubmit, STATGROUP_DriverMonitorAMD);
DECLARE_DWORD_COUNTER_STAT(TEXT("HWChannel SAMU GPCOM | Commands Completed"), STAT_DM_AMD_HWChannelSAMUGPUCOMComplete, STATGROUP_DriverMonitorAMD);
DECLARE_DWORD_COUNTER_STAT(TEXT("HWChannel SAMU GPCOM | Commands Submitted"), STAT_DM_AMD_HWChannelSAMUGPUCOMSubmit, STATGROUP_DriverMonitorAMD);
DECLARE_DWORD_COUNTER_STAT(TEXT("HWChannel SAMU RBI | Commands Completed"), STAT_DM_AMD_HWChannelSAMURBIComplete, STATGROUP_DriverMonitorAMD);
DECLARE_DWORD_COUNTER_STAT(TEXT("HWChannel SAMU RBI | Commands Submitted"), STAT_DM_AMD_HWChannelSAMURBISubmit, STATGROUP_DriverMonitorAMD);


template<typename T>
T GetMacGPUStat(TMap<FString, float> const& Stats, FString StatName)
{
	T Result = (T)0;
	if(Stats.Contains(StatName))
	{
		Result = (T)Stats.FindRef(StatName);
	}
	return Result;
}

void FMacPlatformMisc::UpdateDriverMonitorStatistics(int32 DeviceIndex)
{
	if (DeviceIndex >= 0)
	{
		TArray<FMacGPUDescriptor> const& GPUs = FPlatformMisc::GetGPUDescriptors();
		if (DeviceIndex < GPUs.Num())
		{
			FMacGPUDescriptor const& GPU = GPUs[DeviceIndex];
			TMap<FString, float> Stats = GPU.GetPerformanceStatistics();
			
			float DeviceUtil = GetMacGPUStat<float>(Stats, TEXT("Device Utilization %"));
			SET_FLOAT_STAT(STAT_DriverMonitorDeviceUtilisation, DeviceUtil);
			
			DeviceUtil = GetMacGPUStat<float>(Stats, TEXT("Device Utilization % at cur p-state"));
			SET_FLOAT_STAT(STAT_DM_I_DeviceUtilisationAtPState, DeviceUtil);
			
			DeviceUtil = GetMacGPUStat<float>(Stats, TEXT("Device Unit 0 Utilization %"));
			SET_FLOAT_STAT(STAT_DM_I_Device0Utilisation, DeviceUtil);
			
			DeviceUtil = GetMacGPUStat<float>(Stats, TEXT("Device Unit 1 Utilization %"));
			SET_FLOAT_STAT(STAT_DM_I_Device1Utilisation, DeviceUtil);
			
			DeviceUtil = GetMacGPUStat<float>(Stats, TEXT("Device Unit 2 Utilization %"));
			SET_FLOAT_STAT(STAT_DM_I_Device2Utilisation, DeviceUtil);
			
			DeviceUtil = GetMacGPUStat<float>(Stats, TEXT("Device Unit 3 Utilization %"));
			SET_FLOAT_STAT(STAT_DM_I_Device3Utilisation, DeviceUtil);
			
			int64 Memory = GetMacGPUStat<int64>(Stats, TEXT("vramUsedBytes"));
			SET_MEMORY_STAT(STAT_DriverMonitorVRAMUsedBytes, Memory);
			
			Memory = GetMacGPUStat<int64>(Stats, TEXT("vramFreeBytes"));
			SET_MEMORY_STAT(STAT_DriverMonitorVRAMFreeBytes, Memory);
			
			Memory = GetMacGPUStat<int64>(Stats, TEXT("vramLargestFreeBytes"));
			SET_MEMORY_STAT(STAT_DriverMonitorVRAMLargestFreeBytes, Memory);
			
			Memory = GetMacGPUStat<int64>(Stats, TEXT("inUseVidMemoryBytes"));
			SET_MEMORY_STAT(STAT_DriverMonitorInUseVidMemBytes, Memory);
			
			Memory = GetMacGPUStat<int64>(Stats, TEXT("inUseSysMemoryBytes"));
			SET_MEMORY_STAT(STAT_DriverMonitorInUseSysMemBytes, Memory);
			
			Memory = GetMacGPUStat<int64>(Stats, TEXT("gartSizeBytes"));
			SET_MEMORY_STAT(STAT_DriverMonitorgartSizeBytes, Memory);
			
			Memory = GetMacGPUStat<int64>(Stats, TEXT("gartFreeBytes"));
			SET_MEMORY_STAT(STAT_DriverMonitorgartFreeBytes, Memory);
			
			Memory = GetMacGPUStat<int64>(Stats, TEXT("gartUsedBytes"));
			SET_MEMORY_STAT(STAT_DriverMonitorgartUsedBytes, Memory);
			
			Memory = GetMacGPUStat<int64>(Stats, TEXT("gartMapInBytesPerSample"));
			SET_MEMORY_STAT(STAT_DriverMonitorgartMapInBytesPerSample, Memory);
			
			Memory = GetMacGPUStat<int64>(Stats, TEXT("gartMapOutBytesPerSample"));
			SET_MEMORY_STAT(STAT_DriverMonitorgartMapOutBytesPerSample, Memory);
			
			int64 Cycles = GetMacGPUStat<int64>(Stats, TEXT("hardwareWaitTime"));
			SET_CYCLE_COUNTER(STAT_DriverMonitorCPUWaitForGPU, Cycles);
			
			Cycles = GetMacGPUStat<int64>(Stats, TEXT("hardwareSubmitWaitTime"));
			SET_CYCLE_COUNTER(STAT_DriverMonitorCPUWaitToSubmit, Cycles);
			
			Cycles = GetMacGPUStat<int64>(Stats, TEXT("surfaceReadLockIdleWaitTime"));
			SET_CYCLE_COUNTER(STAT_DriverMonitorCPUWaitToSurfaceRead, Cycles);
			
			Cycles = GetMacGPUStat<int64>(Stats, TEXT("surfaceCopyOutWaitTime"));
			SET_CYCLE_COUNTER(STAT_DriverMonitorCPUWaitToSurfacePageOff, Cycles);
			
			Cycles = GetMacGPUStat<int64>(Stats, TEXT("surfaceCopyInWaitTime"));
			SET_CYCLE_COUNTER(STAT_DriverMonitorCPUWaitToSurfacePageOn, Cycles);
			
			Cycles = GetMacGPUStat<int64>(Stats, TEXT("freeSurfaceBackingWaitTime"));
			SET_CYCLE_COUNTER(STAT_DriverMonitorCPUWaitToReclaimSurfaceGART, Cycles);
			
			Cycles = GetMacGPUStat<int64>(Stats, TEXT("vramEvictionWaitTime"));
			SET_CYCLE_COUNTER(STAT_DriverMonitorCPUWaitToVRAMEvict, Cycles);
			
			Cycles = GetMacGPUStat<int64>(Stats, TEXT("freeDataBufferWaitTime"));
			SET_CYCLE_COUNTER(STAT_DriverMonitorCPUWaitToFreeDataBuffer, Cycles);
			
			Memory = GetMacGPUStat<int64>(Stats, TEXT("texturePageOutBytes"));
			SET_MEMORY_STAT(STAT_DriverMonitortexturePageOutBytes, Memory);
			
			Memory = GetMacGPUStat<int64>(Stats, TEXT("textureReadOutBytes"));
			SET_MEMORY_STAT(STAT_DriverMonitortextureReadOutBytes, Memory);
			
			Memory = GetMacGPUStat<int64>(Stats, TEXT("textureVolunteerUnloadBytes"));
			SET_MEMORY_STAT(STAT_DriverMonitortextureVolunteerUnloadBytes, Memory);
			
			Memory = GetMacGPUStat<int64>(Stats, TEXT("agpTextureCreationBytes"));
			SET_MEMORY_STAT(STAT_DriverMonitoragpTextureCreationBytes, Memory);
			
			uint32 Dword = GetMacGPUStat<int32>(Stats, TEXT("agpTextureCreationCount"));
			SET_DWORD_STAT(STAT_DriverMonitoragpTextureCreationCount, Dword);
			
			Memory = GetMacGPUStat<int64>(Stats, TEXT("agprefTextureCreationBytes"));
			SET_MEMORY_STAT(STAT_DriverMonitoragprefTextureCreationBytes, Memory);
			
			Dword = GetMacGPUStat<int32>(Stats, TEXT("agprefTextureCreationCount"));
			SET_DWORD_STAT(STAT_DriverMonitoragprefTextureCreationCount, Dword);
			
			Memory = GetMacGPUStat<int64>(Stats, TEXT("ioSurfacePageInBytes"));
			SET_MEMORY_STAT(STAT_DriverMonitorioSurfacePageInBytes, Memory);
			
			Memory = GetMacGPUStat<int64>(Stats, TEXT("ioSurfacePageOutBytes"));
			SET_MEMORY_STAT(STAT_DriverMonitorioSurfacePageOutBytes, Memory);
			
			Memory = GetMacGPUStat<int64>(Stats, TEXT("ioSurfaceReadOutBytes"));
			SET_MEMORY_STAT(STAT_DriverMonitorioSurfaceReadOutBytes, Memory);
			
			Memory = GetMacGPUStat<int64>(Stats, TEXT("iosurfaceTextureCreationBytes"));
			SET_MEMORY_STAT(STAT_DriverMonitoriosurfaceTextureCreationBytes, Memory);
			
			Dword = GetMacGPUStat<int32>(Stats, TEXT("iosurfaceTextureCreationCount"));
			SET_DWORD_STAT(STAT_DriverMonitoriosurfaceTextureCreationCount, Dword);
			
			Memory = GetMacGPUStat<int64>(Stats, TEXT("oolTextureCreationBytes"));
			SET_MEMORY_STAT(STAT_DriverMonitoroolTextureCreationBytes, Memory);
			
			Memory = GetMacGPUStat<int64>(Stats, TEXT("oolTexturePageInBytes"));
			SET_MEMORY_STAT(STAT_DriverMonitoroolTexturePageInBytes, Memory);
			
			Dword = GetMacGPUStat<int32>(Stats, TEXT("oolTextureCreationCount"));
			SET_DWORD_STAT(STAT_DriverMonitoroolTextureCreationCount, Dword);
			
			Memory = GetMacGPUStat<int64>(Stats, TEXT("orphanedNonReusableSysMemoryBytes"));
			SET_MEMORY_STAT(STAT_DriverMonitororphanedNonReusableSysMemoryBytes, Memory);
			
			Dword = GetMacGPUStat<int32>(Stats, TEXT("orphanedNonReusableSysMemoryCount"));
			SET_DWORD_STAT(STAT_DriverMonitororphanedNonReusableSysMemoryCount, Dword);
			
			Memory = GetMacGPUStat<int64>(Stats, TEXT("orphanedReusableSysMemoryBytes"));
			SET_MEMORY_STAT(STAT_DriverMonitororphanedReusableSysMemoryBytes, Memory);
			
			Dword = GetMacGPUStat<int32>(Stats, TEXT("orphanedReusableSysMemoryCount"));
			SET_DWORD_STAT(STAT_DriverMonitororphanedReusableSysMemoryCount, Dword);
			
			float HitRate = GetMacGPUStat<float>(Stats, TEXT("orphanedReusableSysMemoryHitRate"));
			SET_FLOAT_STAT(STAT_DriverMonitororphanedReusableSysMemoryHitRate, HitRate);
			
			Memory = GetMacGPUStat<int64>(Stats, TEXT("orphanedNonReusableVidMemoryBytes"));
			SET_MEMORY_STAT(STAT_DriverMonitororphanedNonReusableVidMemoryBytes, Memory);
			
			Dword = GetMacGPUStat<int32>(Stats, TEXT("orphanedNonReusableVidMemoryCount"));
			SET_DWORD_STAT(STAT_DriverMonitororphanedNonReusableVidMemoryCount, Dword);
			
			Memory = GetMacGPUStat<int64>(Stats, TEXT("orphanedReusableVidMemoryBytes"));
			SET_MEMORY_STAT(STAT_DriverMonitororphanedReusableVidMemoryBytes, Memory);
			
			Dword = GetMacGPUStat<int32>(Stats, TEXT("orphanedReusableVidMemoryCount"));
			SET_DWORD_STAT(STAT_DriverMonitororphanedReusableVidMemoryCount, Dword);
			
			HitRate = GetMacGPUStat<float>(Stats, TEXT("orphanedReusableVidMemoryHitRate"));
			SET_FLOAT_STAT(STAT_DriverMonitororphanedReusableVidMemoryHitRate, HitRate);
			
			Memory = GetMacGPUStat<int64>(Stats, TEXT("stdTextureCreationBytes"));
			SET_MEMORY_STAT(STAT_DriverMonitorstdTextureCreationBytes, Memory);
			
			Dword = GetMacGPUStat<int32>(Stats, TEXT("stdTextureCreationCount"));
			SET_DWORD_STAT(STAT_DriverMonitorstdTextureCreationCount, Dword);
			
			Memory = GetMacGPUStat<int64>(Stats, TEXT("stdTexturePageInBytes"));
			SET_MEMORY_STAT(STAT_DriverMonitorstdTexturePageInBytes, Memory);
			
			Memory = GetMacGPUStat<int64>(Stats, TEXT("surfaceBufferPageInBytes"));
			SET_MEMORY_STAT(STAT_DriverMonitorsurfaceBufferPageInBytes, Memory);
			
			Memory = GetMacGPUStat<int64>(Stats, TEXT("surfaceBufferPageOutBytes"));
			SET_MEMORY_STAT(STAT_DriverMonitorsurfaceBufferPageOutBytes, Memory);
			
			Memory = GetMacGPUStat<int64>(Stats, TEXT("surfaceBufferReadOutBytes"));
			SET_MEMORY_STAT(STAT_DriverMonitorsurfaceBufferReadOutBytes, Memory);
			
			Dword = GetMacGPUStat<int32>(Stats, TEXT("surfaceTextureCreationCount"));
			SET_DWORD_STAT(STAT_DriverMonitorsurfaceTextureCreationCount, Dword);
			
			Dword = GetMacGPUStat<int32>(Stats, TEXT("surfaceCount"));
			SET_DWORD_STAT(STAT_DriverMonitorSurfaceCount, Dword);
			
			Dword = GetMacGPUStat<int32>(Stats, TEXT("textureCount"));
			SET_DWORD_STAT(STAT_DriverMonitorTextureCount, Dword);
			
			HitRate = GetMacGPUStat<float>(Stats, TEXT("GPU Core Utilization"));
			SET_FLOAT_STAT(STAT_DM_NV_GPUCoreUtilization, HitRate);
			
			HitRate = GetMacGPUStat<float>(Stats, TEXT("GPU Memory Utilization"));
			SET_FLOAT_STAT(STAT_DM_NV_GPUMemoryUtilization, HitRate);
			
			Dword = GetMacGPUStat<int32>(Stats, TEXT("HWChannel C0 | Commands Completed"));
			SET_DWORD_STAT(STAT_DM_AMD_HWChannelC0Complete, Dword);
			
			Dword = GetMacGPUStat<int32>(Stats, TEXT("HWChannel C0 | Commands Submitted"));
			SET_DWORD_STAT(STAT_DM_AMD_HWChannelC0Submit, Dword);
			
			Dword = GetMacGPUStat<int32>(Stats, TEXT("HWChannel C1 | Commands Completed"));
			SET_DWORD_STAT(STAT_DM_AMD_HWChannelC1Complete, Dword);
			
			Dword = GetMacGPUStat<int32>(Stats, TEXT("HWChannel C1 | Commands Submitted"));
			SET_DWORD_STAT(STAT_DM_AMD_HWChannelC1Submit, Dword);
			
			Dword = GetMacGPUStat<int32>(Stats, TEXT("HWChannel DMA0 | Commands Completed"));
			if(!Dword)
			{
				Dword = GetMacGPUStat<int32>(Stats, TEXT("HWChannel sDMA0 | Commands Completed"));
			}
			SET_DWORD_STAT(STAT_DM_AMD_HWChannelDMA0Complete, Dword);
			
			Dword = GetMacGPUStat<int32>(Stats, TEXT("HWChannel DMA0 | Commands Submitted"));
			if(!Dword)
			{
				Dword = GetMacGPUStat<int32>(Stats, TEXT("HWChannel sDMA0 | Commands Submitted"));
			}
			SET_DWORD_STAT(STAT_DM_AMD_HWChannelDMA0Submit, Dword);
			
			Dword = GetMacGPUStat<int32>(Stats, TEXT("HWChannel DMA1 | Commands Completed"));
			if(!Dword)
			{
				Dword = GetMacGPUStat<int32>(Stats, TEXT("HWChannel sDMA1 | Commands Completed"));
			}
			SET_DWORD_STAT(STAT_DM_AMD_HWChannelDMA1Complete, Dword);
			
			Dword = GetMacGPUStat<int32>(Stats, TEXT("HWChannel DMA1 | Commands Submitted"));
			if (!Dword)
			{
				Dword = GetMacGPUStat<int32>(Stats, TEXT("HWChannel sDMA1 | Commands Submitted"));
			}
			SET_DWORD_STAT(STAT_DM_AMD_HWChannelDMA1Submit, Dword);
			
			Dword = GetMacGPUStat<int32>(Stats, TEXT("HWChannel GFX | Commands Completed"));
			SET_DWORD_STAT(STAT_DM_AMD_HWChannelGFXComplete, Dword);
			
			Dword = GetMacGPUStat<int32>(Stats, TEXT("HWChannel GFX | Commands Submitted"));
			SET_DWORD_STAT(STAT_DM_AMD_HWChannelGFXSubmit, Dword);
			
			Dword = GetMacGPUStat<int32>(Stats, TEXT("HWChannel SPU | Commands Completed"));
			SET_DWORD_STAT(STAT_DM_AMD_HWChannelSPUComplete, Dword);
			
			Dword = GetMacGPUStat<int32>(Stats, TEXT("HWChannel SPU | Commands Submitted"));
			SET_DWORD_STAT(STAT_DM_AMD_HWChannelSPUSubmit, Dword);
			
			Dword = GetMacGPUStat<int32>(Stats, TEXT("HWChannel UVD | Commands Completed"));
			SET_DWORD_STAT(STAT_DM_AMD_HWChannelUVDComplete, Dword);
			
			Dword = GetMacGPUStat<int32>(Stats, TEXT("HWChannel UVD | Commands Submitted"));
			SET_DWORD_STAT(STAT_DM_AMD_HWChannelUVDSubmit, Dword);
			
			Dword = GetMacGPUStat<int32>(Stats, TEXT("HWChannel VCE | Commands Completed"));
			SET_DWORD_STAT(STAT_DM_AMD_HWChannelVCEComplete, Dword);
			
			Dword = GetMacGPUStat<int32>(Stats, TEXT("HWChannel VCE | Commands Submitted"));
			SET_DWORD_STAT(STAT_DM_AMD_HWChannelVCESubmit, Dword);
			
			Dword = GetMacGPUStat<int32>(Stats, TEXT("HWChannel VCELLQ | Commands Completed"));
			SET_DWORD_STAT(STAT_DM_AMD_HWChannelVCELLQComplete, Dword);
			
			Dword = GetMacGPUStat<int32>(Stats, TEXT("HWChannel VCELLQ | Commands Submitted"));
			SET_DWORD_STAT(STAT_DM_AMD_HWChannelVCELLQSubmit, Dword);
			
			Dword = GetMacGPUStat<int32>(Stats, TEXT("HWChannel KIQ | Commands Completed"));
			SET_DWORD_STAT(STAT_DM_AMD_HWChannelKIQComplete, Dword);

			Dword = GetMacGPUStat<int32>(Stats, TEXT("HWChannel KIQ | Commands Submitted"));
			SET_DWORD_STAT(STAT_DM_AMD_HWChannelKIQSubmit, Dword);
			
			Dword = GetMacGPUStat<int32>(Stats, TEXT("HWChannel SAMU GPCOM | Commands Completed"));
			SET_DWORD_STAT(STAT_DM_AMD_HWChannelSAMUGPUCOMComplete, Dword);
			
			Dword = GetMacGPUStat<int32>(Stats, TEXT("HWChannel SAMU GPCOM | Commands Submitted"));
			SET_DWORD_STAT(STAT_DM_AMD_HWChannelSAMUGPUCOMSubmit, Dword);
			
			Dword = GetMacGPUStat<int32>(Stats, TEXT("HWChannel SAMU RBI | Commands Completed"));
			SET_DWORD_STAT(STAT_DM_AMD_HWChannelSAMURBIComplete, Dword);
			
			Dword = GetMacGPUStat<int32>(Stats, TEXT("HWChannel SAMU RBI | Commands Submitted"));
			SET_DWORD_STAT(STAT_DM_AMD_HWChannelSAMURBISubmit, Dword);
		}
	}
}

int FMacPlatformMisc::GetDefaultStackSize()
{
	// Thread sanitiser requires 5x the memory.
#if __has_feature(thread_sanitizer)
	return 20 * 1024 * 1024;
#else
	return 4 * 1024 * 1024;
#endif
}

IPlatformChunkInstall* FMacPlatformMisc::GetPlatformChunkInstall()
{
	static IPlatformChunkInstall* ChunkInstall = nullptr;
	static bool bIniChecked = false;
	if (!ChunkInstall || !bIniChecked)
	{
		IPlatformChunkInstallModule* PlatformChunkInstallModule = nullptr;
		if (!GEngineIni.IsEmpty())
		{
			FString InstallModule;
			GConfig->GetString(TEXT("StreamingInstall"), TEXT("DefaultProviderName"), InstallModule, GEngineIni);
			FModuleStatus Status;
			if (FModuleManager::Get().QueryModule(*InstallModule, Status))
			{
				PlatformChunkInstallModule = FModuleManager::LoadModulePtr<IPlatformChunkInstallModule>(*InstallModule);
				if (PlatformChunkInstallModule != nullptr)
				{
					// Attempt to grab the platform installer
					ChunkInstall = PlatformChunkInstallModule->GetPlatformChunkInstall();
				}
			}
			bIniChecked = true;
		}

		if (PlatformChunkInstallModule == nullptr)
		{
			// Placeholder instance
			ChunkInstall = FGenericPlatformMisc::GetPlatformChunkInstall();
		}
	}

	return ChunkInstall;
}
