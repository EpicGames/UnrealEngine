// Copyright Epic Games, Inc. All Rights Reserved.

#include "Mac/MacPlatformMutex.h"

#include "HAL/PlatformTime.h"
#include "Misc/App.h"
#include "Misc/DateTime.h"
#include "Apple/PreAppleSystemHeaders.h"
#include <mach-o/dyld.h>
#include <fcntl.h>
#include <libproc.h>
#include <unistd.h>
#include "Apple/PostAppleSystemHeaders.h"

namespace UE
{

FMacSystemWideMutex::FMacSystemWideMutex(const FString& InName, FTimespan InTimeout)
{
	check(InName.Len() > 0);
	check(InTimeout >= FTimespan::Zero());
	check(InTimeout.GetTotalSeconds() < (double)FLT_MAX);

	FString SanitizedName(InName);
	FGenericPlatformMisc::SanitizeSystemWideMutexName(SanitizedName);
	const FString LockPath = FString(FPlatformProcess::ApplicationSettingsDir()) / SanitizedName;
	FileHandle = -1;

	double ExpireTimeSecs = FPlatformTime::Seconds() + InTimeout.GetTotalSeconds();
	while (true)
	{
		if (FileHandle == -1)
		{
			// Try to open the file.
			FileHandle = open(TCHAR_TO_UTF8(*LockPath), O_CREAT | O_WRONLY | O_NONBLOCK, S_IRUSR | S_IWUSR | S_IRGRP | S_IWGRP | S_IROTH | S_IWOTH);
		}

		// If the file is open, try to lock it, but don't block. If the file is already locked by another process or another thread, blocking here may not honor InTimeout.
		if (FileHandle != -1 && flock(FileHandle, LOCK_EX | LOCK_NB) == 0)
		{
			return; // Lock was successfully taken.
		}

		// If the lock isn't acquired and no time is left to retry, clean up and set the state as 'invalid'
		if (InTimeout == FTimespan::Zero() || FPlatformTime::Seconds() > ExpireTimeSecs)
		{
			if (FileHandle != -1)
			{
				close(FileHandle);
				FileHandle = -1;
			}
			return; // Lock wasn't acquired within the allowed time.
		}

		// Either the file did not open or the lock wasn't acquired, retry.
		const float RetrySeconds = FMath::Min((float)InTimeout.GetTotalSeconds(), 0.25f);
		FPlatformProcess::Sleep(RetrySeconds);
	}
}

FMacSystemWideMutex::~FMacSystemWideMutex()
{
	Release();
}

bool FMacSystemWideMutex::IsValid() const
{
	return FileHandle != -1;
}

void FMacSystemWideMutex::Release()
{
	if (IsValid())
	{
		flock(FileHandle, LOCK_UN);
		close(FileHandle);
		FileHandle = -1;
	}
}

} // UE
