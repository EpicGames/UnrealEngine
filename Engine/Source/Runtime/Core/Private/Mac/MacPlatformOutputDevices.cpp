// Copyright Epic Games, Inc. All Rights Reserved.

#include "Mac/MacPlatformOutputDevices.h"
#include "Mac/MacErrorOutputDevice.h"

class FOutputDevice* FMacPlatformOutputDevices::GetEventLog()
{
	return NULL;
}

FOutputDeviceError* FMacPlatformOutputDevices::GetError()
{
	static FMacErrorOutputDevice Singleton;
	return &Singleton;
}

