// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MacPlatformProcess.mm: Mac implementations of Process functions
=============================================================================*/

#include "Mac/MacPlatformProcess.h"
#include "Mac/MacPlatform.h"
#include "Apple/ApplePlatformRunnableThread.h"
#include "Containers/UnrealString.h"
#include "Misc/App.h"
#include "Misc/CoreDelegates.h"
#include "Misc/Paths.h"
#include "Misc/ScopeExit.h"
#include "Misc/StringBuilder.h"
#include "HAL/FileManager.h"
#include "Apple/PreAppleSystemHeaders.h"
#include <mach-o/dyld.h>
#include <mach/thread_act.h>
#include <mach/thread_policy.h>
#include <libproc.h>
#include <spawn.h>

#include "Apple/ScopeAutoreleasePool.h"
#include "Apple/AppleStringUtils.h"
#include "Apple/PostAppleSystemHeaders.h"
#if PLATFORM_MAC_X86
    #include <cpuid.h>
#endif

namespace PlatformProcessLimits
{
	enum
	{
		MaxArgvParameters	 = 256
	};
};

static void* GetDllHandleImpl(NSString* DylibPath, NSString* ExecutableFolder)
{
	SCOPED_AUTORELEASE_POOL;

	// Check if dylib is already loaded
	void* Handle = dlopen([DylibPath fileSystemRepresentation], RTLD_NOLOAD | RTLD_LAZY | RTLD_LOCAL);
	if (!Handle)
	{
		// Maybe it was loaded using RPATH
		NSString* DylibName;
		if ([DylibPath hasPrefix:ExecutableFolder])
		{
			DylibName = [DylibPath substringFromIndex:[ExecutableFolder length] + 1];
		}
		else
		{
			DylibName = [DylibPath lastPathComponent];
		}
		Handle = dlopen([[@"@rpath" stringByAppendingPathComponent:DylibName] fileSystemRepresentation], RTLD_NOLOAD | RTLD_LAZY | RTLD_GLOBAL);
	}
	
	if (!Handle)
	{
		// Not loaded yet, so try to open it
		Handle = dlopen([DylibPath fileSystemRepresentation], RTLD_LAZY | RTLD_GLOBAL);
	}
	
	if (!Handle && FParse::Param(FCommandLine::Get(), TEXT("dllerrors")))
	{
		UE_LOGF(LogMac, Warning, "dlopen failed: %ls", ANSI_TO_TCHAR(dlerror()));
	}
		
	return Handle;
}

void* FMacPlatformProcess::GetDllHandle( const TCHAR* Filename )
{
	SCOPED_AUTORELEASE_POOL;

	check(Filename);

	NSString* DylibPath = FString(Filename).GetNSString();
	NSString* ExecutableFolder = [[[NSBundle mainBundle] executablePath] stringByDeletingLastPathComponent];
	void* Handle = nullptr;

    Handle = GetDllHandleImpl(DylibPath, ExecutableFolder);
    if (!Handle)
    {
        // If it's not a absolute or relative path, try to find the file in the app bundle
        DylibPath = [ExecutableFolder stringByAppendingPathComponent:DylibPath];
        Handle = GetDllHandleImpl(DylibPath, ExecutableFolder);
    }
	return Handle;
}

void FMacPlatformProcess::FreeDllHandle( void* DllHandle )
{
	check( DllHandle );
	dlclose( DllHandle );
}

FString FMacPlatformProcess::GenerateApplicationPath( const FString& AppName, EBuildConfiguration BuildConfiguration)
{
	SCOPED_AUTORELEASE_POOL;
	
	FString PlatformName = "Mac";
	FString ExecutableName = AppName;
	if (BuildConfiguration != EBuildConfiguration::Development)
	{
		ExecutableName += FString::Printf(TEXT("-%s-%s"), *PlatformName, LexToString(BuildConfiguration));
	}

	NSURL* CurrentBundleURL = [[NSBundle mainBundle] bundleURL];
	NSString* CurrentBundleName = [[CurrentBundleURL lastPathComponent] stringByDeletingPathExtension];
	if (FString(CurrentBundleName) == ExecutableName)
	{
		CFStringRef FilePath = CFURLCopyFileSystemPath((CFURLRef)CurrentBundleURL, kCFURLPOSIXPathStyle);
		FString ExecutablePath = FString::Printf(TEXT("%s/Contents/MacOS/%s"), *FString((NSString*)FilePath), *ExecutableName);
		CFRelease(FilePath);
		return ExecutablePath;
	}
	else
	{
		// Try expected path of an executable inside an app package in Engine Binaries
		FString ExecutablePath = FPaths::EngineDir() / FString::Printf(TEXT("Binaries/%s/%s.app/Contents/MacOS/%s"), *PlatformName, *ExecutableName, *ExecutableName);

		NSString* LaunchPath = ExecutablePath.GetNSString();
		
		if ([[NSFileManager defaultManager] fileExistsAtPath: LaunchPath])
		{
			return ExecutablePath;
		}
		else
		{
			// Try the path of a simple executable file in Engine Binaries
			ExecutablePath = FPaths::EngineDir() / FString::Printf(TEXT("Binaries/%s/%s"), *PlatformName, *ExecutableName);
			LaunchPath = ExecutablePath.GetNSString();

			if ([[NSFileManager defaultManager] fileExistsAtPath:LaunchPath])
			{
				return ExecutablePath;
			}
		}
		return FString(); // Not found.
	}
}

void* FMacPlatformProcess::GetDllExport( void* DllHandle, const TCHAR* ProcName )
{
	check(DllHandle);
	check(ProcName);
	return dlsym( DllHandle, TCHAR_TO_ANSI(ProcName) );
}

bool FMacPlatformProcess::CanLaunchURL(const TCHAR* URL)
{
	return URL != nullptr;
}

void FMacPlatformProcess::LaunchURL( const TCHAR* URL, const TCHAR* Parms, FString* Error )
{
	SCOPED_AUTORELEASE_POOL;

	UE_LOGF(LogMac, Log,  "LaunchURL %ls %ls", URL, Parms?Parms:TEXT("") );

	if (FCoreDelegates::ShouldLaunchUrl.IsBound() && !FCoreDelegates::ShouldLaunchUrl.Execute(URL))
	{
		if (Error)
		{
			*Error = TEXT("LaunchURL cancelled by delegate");
		}
		return;
	}

	NSString* Url = (NSString*)FPlatformString::TCHARToCFString( URL );
	
	FString SchemeName;
	bool bHasSchemeName = FParse::SchemeNameFromURI(URL, SchemeName);
		
	NSURL* UrlToOpen = [NSURL URLWithString: bHasSchemeName ? Url : [NSString stringWithFormat: @"http://%@", Url]];
	[[NSWorkspace sharedWorkspace] openURL: UrlToOpen];
	CFRelease( (CFStringRef)Url );
	if( Error )
	{
		*Error = TEXT("");
	}
}

FString FMacPlatformProcess::GetGameBundleId()
{
	return FString([[NSBundle mainBundle] bundleIdentifier]);
}

bool FMacPlatformProcess::ExecProcess( const TCHAR* URL, const TCHAR* Params, int32* OutReturnCode, FString* OutStdOut, FString* OutStdErr, const TCHAR* OptionalWorkingDirectory, bool bShouldEndWithParentProcess)
{
	FString CmdLineParams = Params;
	FString ExecutableFileName = URL;
	int32 ReturnCode = -1;

	void* PipeStdOutRead = nullptr;
	void* PipeStdOutWrite = nullptr;
	verify(FPlatformProcess::CreatePipe(PipeStdOutRead, PipeStdOutWrite));

	void* PipeStdErrRead = nullptr;
	void* PipeStdErrWrite = nullptr;
	verify(FPlatformProcess::CreatePipe(PipeStdErrRead, PipeStdErrWrite));

	bool bInvoked = false;

	const bool bLaunchDetached = true;
	const bool bLaunchHidden = false;
	const bool bLaunchReallyHidden = bLaunchHidden;

	FProcHandle ProcHandle = FPlatformProcess::CreateProcInternal(*ExecutableFileName, *CmdLineParams, bLaunchDetached, bLaunchHidden, bLaunchReallyHidden, NULL, 0, OptionalWorkingDirectory, PipeStdOutWrite, nullptr, PipeStdErrWrite, false);
	if (ProcHandle.IsValid())
	{
		while (FPlatformProcess::IsProcRunning(ProcHandle))
		{
			FString NewLineStdOut = FPlatformProcess::ReadPipe(PipeStdOutRead);
			if (NewLineStdOut.Len() > 0)
			{
				if (OutStdOut != nullptr)
				{
					*OutStdOut += NewLineStdOut;
				}
			}
			FString NewLineStdErr = FPlatformProcess::ReadPipe(PipeStdErrRead);
			if (NewLineStdErr.Len() > 0)
			{
				if (OutStdErr != nullptr)
				{
					*OutStdErr += NewLineStdErr;
				}
			}
			FPlatformProcess::Sleep(0.0);
		}

		// read the remainder
		for(;;)
		{
			FString NewLineStdOut = FPlatformProcess::ReadPipe(PipeStdOutRead);
			if (NewLineStdOut.Len() <= 0)
			{
				break;
			}

			if (OutStdOut != nullptr)
			{
				*OutStdOut += NewLineStdOut;
			}
		}

		for(;;)
		{
			FString NewLineStdErr = FPlatformProcess::ReadPipe(PipeStdErrRead);
			if (NewLineStdErr.Len() <= 0)
			{
				break;
			}

			if (OutStdErr != nullptr)
			{
				*OutStdErr += NewLineStdErr;
			}
		}

		FPlatformProcess::Sleep(0.0);

		bInvoked = true;
		bool bGotReturnCode = FPlatformProcess::GetProcReturnCode(ProcHandle, &ReturnCode);
		check(bGotReturnCode)
		if (OutReturnCode != nullptr)
		{
			*OutReturnCode = ReturnCode;
		}

		FPlatformProcess::CloseProc(ProcHandle);
	}
	else
	{
		bInvoked = false;
		if (OutReturnCode != nullptr)
		{
			*OutReturnCode = -1;
		}
		if (OutStdOut != nullptr)
		{
			*OutStdOut = "";
		}
		UE_LOGF(LogHAL, Warning, "Failed to launch Tool. (%ls)", *ExecutableFileName);
	}
	FPlatformProcess::ClosePipe(PipeStdOutRead, PipeStdOutWrite);
	FPlatformProcess::ClosePipe(PipeStdErrRead, PipeStdErrWrite);
	return bInvoked;
}

FProcHandle FMacPlatformProcess::CreateProc(const TCHAR* URL, const TCHAR* Parms, bool bLaunchDetached, bool bLaunchHidden, bool bLaunchReallyHidden, uint32* OutProcessID, int32 PriorityModifier, const TCHAR* OptionalWorkingDirectory, void* PipeWriteChild, void* PipeReadChild)
{
	return CreateProcInternal(URL, Parms, bLaunchDetached, bLaunchHidden, bLaunchReallyHidden, OutProcessID, PriorityModifier, OptionalWorkingDirectory, PipeWriteChild, PipeReadChild, PipeWriteChild, false);
}

FProcHandle FMacPlatformProcess::CreateProc(const TCHAR* URL, const TCHAR* Parms, bool bLaunchDetached, bool bLaunchHidden, bool bLaunchReallyHidden, uint32* OutProcessID, int32 PriorityModifier, const TCHAR* OptionalWorkingDirectory, void* PipeWriteChild, void* PipeReadChild, void* PipeStdErrChild)
{
	return CreateProcInternal(URL, Parms, bLaunchDetached, bLaunchHidden, bLaunchReallyHidden, OutProcessID, PriorityModifier, OptionalWorkingDirectory, PipeWriteChild, PipeReadChild, PipeStdErrChild, false);
}

TTuple<FProcHandle, UE::HAL::EProcessId> FMacPlatformProcess::CreateProc(const UE::HAL::FProcessStartInfo& StartInfo)
{
	uint32 Id;
	FProcHandle Handle = CreateProcInternal(StartInfo.Uri, StartInfo.Arguments, StartInfo.bDetached, false, StartInfo.bHidden, &Id, StartInfo.PriorityModifier, StartInfo.WorkingDirectory, StartInfo.StdOut, StartInfo.StdIn, StartInfo.StdErr, StartInfo.bInheritHandles);
	return {Handle, UE::HAL::EProcessId{Id}};
}

FProcHandle FMacPlatformProcess::CreateProcInternal(const TCHAR* URL, const TCHAR* Parms, bool bLaunchDetached, bool bLaunchHidden, bool bLaunchReallyHidden, uint32* OutProcessID, int32 PriorityModifier, const TCHAR* OptionalWorkingDirectory, void* PipeStdOutChild, void* PipeStdInChild, void* PipeStdErrChild, bool bInheritHandles)
{
	check(URL);
	check(URL[0] != '\"');
	check(Parms);

	SCOPED_AUTORELEASE_POOL;

	// @TODO bLaunchHidden bLaunchReallyHidden are not handled
	// We need an absolute path to executable
	FString ProcessPath = URL;
	if (*URL != TEXT('/'))
	{
		ProcessPath = FPaths::ConvertRelativePathToFull(ProcessPath);
	}

	// For programs that are wrapped in an App container
	{
		NSString* nsProcessPath = ProcessPath.GetNSString();
		if (![[NSFileManager defaultManager] fileExistsAtPath: nsProcessPath])
		{
			NSString* AppName = [[nsProcessPath lastPathComponent] stringByDeletingPathExtension];
			nsProcessPath = [[[NSWorkspace sharedWorkspace] URLForApplicationWithBundleIdentifier:AppName] path];
		}
		
		if ([[NSFileManager defaultManager] fileExistsAtPath: nsProcessPath])
		{
			if([[NSWorkspace sharedWorkspace] isFilePackageAtPath: nsProcessPath])
			{
				NSBundle* Bundle = [NSBundle bundleWithPath:nsProcessPath];
				if(Bundle != nil)
				{
					nsProcessPath = [Bundle executablePath];
					if(nsProcessPath != nil)
					{
						ProcessPath = nsProcessPath;
					}
				}
			}
		}
	}

	if (!FPaths::FileExists(ProcessPath))
	{
		return FProcHandle();
	}

	FString Commandline = FString::Printf(TEXT("\"%s\""), *ProcessPath);
	Commandline += TEXT(" ");
	Commandline += Parms;

	UE_LOGF(LogHAL, Verbose, "FMacPlatformProcess::CreateProc: '%ls'", *Commandline);

	TArray<FString> ArgvArray;
	int Argc = Commandline.ParseIntoArray(ArgvArray, TEXT(" "), true);
	char* Argv[PlatformProcessLimits::MaxArgvParameters + 1] = { NULL };	// last argument is NULL, hence +1
	struct CleanupArgvOnExit
	{
		int Argc;
		char** Argv;	// relying on it being long enough to hold Argc elements

		CleanupArgvOnExit( int InArgc, char *InArgv[] )
			:	Argc(InArgc)
			,	Argv(InArgv)
		{}

		~CleanupArgvOnExit()
		{
			for (int Idx = 0; Idx < Argc; ++Idx)
			{
				FMemory::Free(Argv[Idx]);
			}
		}
	} CleanupGuard(Argc, Argv);

	// make sure we do not lose arguments with spaces in them due to Commandline.ParseIntoArray breaking them apart above
	// @todo this code might need to be optimized somehow and integrated with main argument parser below it
	TArray<FString> NewArgvArray;
	if (Argc > 0)
	{
		if (Argc > PlatformProcessLimits::MaxArgvParameters)
		{
			UE_LOGF(LogHAL, Warning, "FMacPlatformProcess::CreateProc: too many (%d) commandline arguments passed, will only pass %d",
				Argc, PlatformProcessLimits::MaxArgvParameters);
			Argc = PlatformProcessLimits::MaxArgvParameters;
		}

		FString MultiPartArg;
		for (int32 Index = 0; Index < Argc; Index++)
		{
			if (MultiPartArg.IsEmpty())
			{
				if ((ArgvArray[Index].StartsWith(TEXT("\"")) && !ArgvArray[Index].EndsWith(TEXT("\""))) // check for a starting quote but no ending quote, excludes quoted single arguments
					|| (ArgvArray[Index].Contains(TEXT("=\"")) && !ArgvArray[Index].EndsWith(TEXT("\""))) // check for quote after =, but no ending quote, this gets arguments of the type -blah="string string string"
					|| ArgvArray[Index].EndsWith(TEXT("=\""))) // check for ending quote after =, this gets arguments of the type -blah=" string string string "
				{
					MultiPartArg = ArgvArray[Index];
				}
				else
				{
					if (ArgvArray[Index].Contains(TEXT("=\"")))
					{
						FString SingleArg = ArgvArray[Index];
						SingleArg = SingleArg.Replace(TEXT("=\""), TEXT("="));
						NewArgvArray.Add(SingleArg.TrimQuotes(NULL));
					}
					else
					{
						NewArgvArray.Add(ArgvArray[Index].TrimQuotes(NULL));
					}
				}
			}
			else
			{
				MultiPartArg += TEXT(" ");
				MultiPartArg += ArgvArray[Index];
				if (ArgvArray[Index].EndsWith(TEXT("\"")))
				{
					if (MultiPartArg.StartsWith(TEXT("\"")))
					{
						NewArgvArray.Add(MultiPartArg.TrimQuotes(NULL));
					}
					else if (MultiPartArg.Contains(TEXT("=\"")))
					{
						FString SingleArg = MultiPartArg.Replace(TEXT("=\""), TEXT("="));
						NewArgvArray.Add(SingleArg.TrimQuotes(nullptr));
					}
					else
					{
						NewArgvArray.Add(MultiPartArg);
					}
					MultiPartArg.Empty();
				}
			}
		}
	}
	// update Argc with the new argument count
	Argc = NewArgvArray.Num();

	if (Argc > 0)	// almost always, unless there's no program name
	{
		if (Argc > PlatformProcessLimits::MaxArgvParameters)
		{
			UE_LOGF(LogHAL, Warning, "FMacPlatformProcess::CreateProc: too many (%d) commandline arguments passed, will only pass %d",
				Argc, PlatformProcessLimits::MaxArgvParameters);
			Argc = PlatformProcessLimits::MaxArgvParameters;
		}

		for (int Idx = 0; Idx < Argc; ++Idx)
		{
			FTCHARToUTF8 AnsiBuffer(*NewArgvArray[Idx]);
			const char* Ansi = AnsiBuffer.Get();
			size_t AnsiSize = FCStringAnsi::Strlen(Ansi) + 1;	// will work correctly with UTF-8
			check(AnsiSize);

			Argv[Idx] = reinterpret_cast< char* >( FMemory::Malloc(AnsiSize) );
			check(Argv[Idx]);

			FCStringAnsi::Strncpy(Argv[Idx], Ansi, AnsiSize);	// will work correctly with UTF-8
		}

		// last Argv should be NULL
		check(Argc <= PlatformProcessLimits::MaxArgvParameters + 1);
		Argv[Argc] = NULL;
	}

	extern char ** environ;	// provided by libc
	pid_t ChildPid = -1;

	posix_spawnattr_t SpawnAttr;
	posix_spawnattr_init(&SpawnAttr);
	short int SpawnFlags = !bInheritHandles ? POSIX_SPAWN_CLOEXEC_DEFAULT : 0;

	// Makes spawned processes have its own unique group id so we can kill the entire group without killing the parent
	SpawnFlags |= POSIX_SPAWN_SETPGROUP;

	// - These are the extra environment keys when turning on GPU Frame Capture via Xcode 11.3 in macOS Catalina (10.15.2):
	//		DYLD_INSERT_LIBRARIES, DYMTL_TOOLS_DYLIB_PATH, GPUTOOLS_LOAD_GTMTLCAPTURE, GT_HOST_URL_MTL and METAL_LOAD_INTERPOSER
	// - Both DYLD_INSERT_LIBRARIES and METAL_LOAD_INTERPOSER seem to be new for Catalina (10.15.2) compared to Mojave (10.14.6).
	// - Using DYLD_INSERT_LIBRARIES seem to be causing a stall at child process startup with Xcode debugger attached to the parent process.
	// - Removing DYLD_INSERT_LIBRARIES removes the stall for child process startup which is especially useful for ShaderCompileWorker when invoking MetalCompiler and it's other child processes tools.
	char** EnvVariables = environ;
	if (WITH_EDITOR && FPlatformMisc::IsDebuggerPresent())
	{
		int32 NumEnvVariables = 0;
		int32 DyldInsertLibrariesEnvVarIndex = -1;

		while (environ[NumEnvVariables])
		{
			if (FCStringAnsi::Strstr(environ[NumEnvVariables], "DYLD_INSERT_LIBRARIES=") == environ[NumEnvVariables])
			{
				DyldInsertLibrariesEnvVarIndex = NumEnvVariables;
			}
			++NumEnvVariables;
		}

		if (DyldInsertLibrariesEnvVarIndex != -1)
		{
			EnvVariables = (char**)FMemory::Malloc(sizeof(char*) * NumEnvVariables + 1);

			int32 NewCount = 0;
			for (int32 VarIndex = 0; VarIndex < NumEnvVariables; ++VarIndex)
			{
				if (VarIndex != DyldInsertLibrariesEnvVarIndex)
				{
					EnvVariables[NewCount++] = environ[VarIndex];
				}
			}
			EnvVariables[NewCount] = nullptr;
		}
	}

	posix_spawn_file_actions_t FileActions;
	posix_spawn_file_actions_init(&FileActions);

	if (PipeStdOutChild)
	{
		posix_spawn_file_actions_adddup2(&FileActions, [(NSFileHandle*)PipeStdOutChild fileDescriptor], STDOUT_FILENO);
	}
	else if (!bInheritHandles)
	{
		if (bLaunchDetached)
		{
			posix_spawn_file_actions_addopen(&FileActions, STDOUT_FILENO, "/dev/null", O_RDWR, 0);
		}
		else
		{
			posix_spawn_file_actions_addinherit_np(&FileActions, STDOUT_FILENO);
		}
	}

	if (PipeStdErrChild)
	{
		posix_spawn_file_actions_adddup2(&FileActions, [(NSFileHandle*)PipeStdErrChild fileDescriptor], STDERR_FILENO);
	}
	else if (!bInheritHandles)
	{
		if (bLaunchDetached)
		{
			posix_spawn_file_actions_addopen(&FileActions, STDERR_FILENO, "/dev/null", O_RDWR, 0);
		}
		else
		{
			posix_spawn_file_actions_addinherit_np(&FileActions, STDERR_FILENO);
		}
	}

	if (PipeStdInChild)
	{
		posix_spawn_file_actions_adddup2(&FileActions, [(NSFileHandle*)PipeStdInChild fileDescriptor], STDIN_FILENO);
	}
	else if (!bInheritHandles)
	{
		if (bLaunchDetached)
		{
			posix_spawn_file_actions_addopen(&FileActions, STDIN_FILENO, "/dev/null", O_RDONLY, 0);
		}
		else
		{
			posix_spawn_file_actions_addinherit_np(&FileActions, STDIN_FILENO);
		}
	}

	if (OptionalWorkingDirectory)
	{
		posix_spawn_file_actions_addchdir_np(&FileActions, TCHAR_TO_UTF8(OptionalWorkingDirectory));
	}

	posix_spawnattr_setflags(&SpawnAttr, SpawnFlags);
	int PosixSpawnErrNo = posix_spawn(&ChildPid, TCHAR_TO_UTF8(*ProcessPath), &FileActions, &SpawnAttr, Argv, EnvVariables);
	posix_spawn_file_actions_destroy(&FileActions);

	posix_spawnattr_destroy(&SpawnAttr);

	// Free the allocated memory if we modified the env variables instead of using environ directly
	if (EnvVariables != environ)
	{
		FMemory::Free(EnvVariables);
	}

	if (PosixSpawnErrNo != 0)
	{
		UE_LOGF(LogHAL, Warning, "FMacPlatformProcess::CreateProc: posix_spawn(\"%ls\") failed (%d, %ls)", *ProcessPath, PosixSpawnErrNo, UTF8_TO_TCHAR(strerror(PosixSpawnErrNo)));
		return FProcHandle();
	}

	if (PriorityModifier != 0)
	{
		PriorityModifier = MIN(PriorityModifier, -2);
		PriorityModifier = MAX(PriorityModifier, 2);
		// priority values: 20 = lowest, 10 = low, 0 = normal, -10 = high, -20 = highest
		setpriority(PRIO_PROCESS, ChildPid, -PriorityModifier * 10);
	}

	if (OutProcessID)
	{
		*OutProcessID = ChildPid;
	}

	return FProcHandle(new FProcState(ChildPid));
}

/*
 * Return a limited use FProcHandle from a PID. Currently can only use w/ IsProcRunning().
 *
 * WARNING (from Arciel): PIDs can and will be reused. We have had that issue
 * before: the editor was tracking ShaderCompileWorker by their PIDs, and a
 * long-running process (something from PS4 SDK) got a reused SCW PID,
 * resulting in compilation never ending.
 */
FProcHandle FMacPlatformProcess::OpenProcess(uint32 ProcessID)
{
	pid_t Pid = static_cast< pid_t >(ProcessID);

	// check if actually running
	int KillResult = kill(Pid, 0);	// no actual signal is sent
	check(KillResult != -1 || errno != EINVAL);

	// errno == EPERM: don't have permissions to send signal
	// errno == ESRCH: proc doesn't exist
	bool bIsRunning = (KillResult == 0);
	return FProcHandle(bIsRunning ? Pid : -1);
}

namespace UE::Mac
{
	namespace
	{
		bool bProcessReaperStarted = false;
		// Using semaphore_t as it's async-signal-safe
		semaphore_t ProcessReaperSemaphore;
		std::atomic_bool bProcessReaperSignaled = false;
		// Using pthread_mutex_t so it's not dynamic initialized
		pthread_mutex_t ProcessesToReapMutex = PTHREAD_MUTEX_INITIALIZER;
		// [[clang::no_destroy]] required here to not potentially crash on exit
		[[clang::no_destroy]] TSet<pid_t> ProcessesToReap;
		
		void ReleaseProcessReaper()
		{
			if (!bProcessReaperSignaled.exchange(true, std::memory_order_seq_cst))
			{
				verify(semaphore_signal(ProcessReaperSemaphore) == KERN_SUCCESS);
			}
		}
		
		void AcquireProcessReaper()
		{
			verify(semaphore_wait(ProcessReaperSemaphore) == KERN_SUCCESS);
			bProcessReaperSignaled.store(false, std::memory_order_seq_cst);
		}
	}

	void StartProcessReaper()
	{
		if (bProcessReaperStarted)
		{
			return;
		}
		
		bProcessReaperStarted = true;
		
		verify(semaphore_create(mach_task_self(), &ProcessReaperSemaphore, SYNC_POLICY_FIFO, 0) == KERN_SUCCESS);

		// Using pthread_t so it runs forever
		pthread_t Thread;
		verify(pthread_create(&Thread, nullptr, [](void*) -> void*
		{
			auto Get = []() -> pid_t
			{
				siginfo_t SigInfo;
				SigInfo.si_pid = 0;
				while (waitid(P_ALL, 0, &SigInfo, WEXITED | WNOHANG | WNOWAIT) && errno == EINTR);
				return SigInfo.si_pid;
			};
			
			auto Reap = [](pid_t PId) -> bool
			{
				do
				{
					if (pid_t Result = waitpid(PId, nullptr, WNOHANG); Result >= 0)
					{
						return Result;
					}
				}
				while (errno == EINTR);
				return false;
			};
			
			for (;;)
			{
				AcquireProcessReaper();
				
				while (pid_t PId = Get())
				{
					verify(pthread_mutex_lock(&ProcessesToReapMutex) == 0);
					ON_SCOPE_EXIT
					{
						verify(pthread_mutex_unlock(&ProcessesToReapMutex) == 0);
					};

					if (!ProcessesToReap.Remove(PId))
					{
						for (TSet<pid_t>::TIterator Iterator = ProcessesToReap.CreateIterator(); Iterator; ++Iterator)
						{
							if (Reap(*Iterator))
							{
								Iterator.RemoveCurrent();
							}
						}
						break;
					}

					verify(Reap(PId));
				}
			}
		}, nullptr) == 0);
		
		struct sigaction Action;
		Action.sa_handler = [](int)
		{
			ReleaseProcessReaper();
		};
		sigemptyset(&Action.sa_mask);
		Action.sa_flags = SA_NOCLDSTOP | SA_RESTART;
		verify(sigaction(SIGCHLD, &Action, nullptr) == 0);
	}

	void AddToProcessReaper(pid_t PId)
	{
		verify(pthread_mutex_lock(&ProcessesToReapMutex) == 0);
		ON_SCOPE_EXIT
		{
			verify(pthread_mutex_unlock(&ProcessesToReapMutex) == 0);
		};

		StartProcessReaper();
		
		ProcessesToReap.Add(PId);
		ReleaseProcessReaper();
	}
}

/** Initialization constructor. */
FProcState::FProcState(pid_t InProcessId)
:	ProcessId{InProcessId}
{
	check(InProcessId);
}

FProcState::~FProcState()
{
	if (IsRunning())
	{
		UE::Mac::AddToProcessReaper(ProcessId);
	}
}

bool FProcState::IsRunning()
{
	if (bIsRunning)
	{
		check(!bHasBeenWaitedFor);	// check for the sake of internal consistency

		// check if actually running
		int KillResult = kill(GetProcessId(), 0);	// no actual signal is sent
		check(KillResult != -1 || errno != EINVAL);

		bIsRunning = (KillResult == 0 || (KillResult == -1 && errno == EPERM));

		// additional check if it's a zombie
		if (bIsRunning)
		{
			for(;;)	// infinite loop in case we get EINTR and have to repeat
			{
				siginfo_t SignalInfo;
				SignalInfo.si_pid = 0;	// if remains 0, treat as child was not waitable (i.e. was running)
				if (waitid(P_PID, GetProcessId(), &SignalInfo, WEXITED | WNOHANG | WNOWAIT))
				{
					if (errno != EINTR)
					{
						int ErrNo = errno;
						UE_LOGF(LogHAL, Fatal, "FMacPlatformProcess::WaitForProc: waitid for pid %d failed (errno=%d, %ls)",
							static_cast< int32 >(GetProcessId()), ErrNo, UTF8_TO_TCHAR(strerror(ErrNo)));
						break;	// exit the loop if for some reason Fatal log (above) returns
					}
				}
				else
				{
					bIsRunning = ( SignalInfo.si_pid != GetProcessId() );
					break;
				}
			}
		}

		// If child is a zombie, wait() immediately to free up kernel resources. Higher level code
		// (e.g. shader compiling manager) can hold on to handle of no longer running process for longer,
		// which is a dubious, but valid behavior. We don't want to keep zombie around though.
		if (!bIsRunning)
		{
			UE_LOGF(LogHAL, Verbose, "Child %d is no longer running (zombie), Wait()ing immediately.", GetProcessId() );
			Wait();
		}
	}

	return bIsRunning;
}

bool FProcState::GetReturnCode(int32* ReturnCodePtr)
{
	check(!bIsRunning || !"You cannot get a return code of a running process");
	if (!bHasBeenWaitedFor)
	{
		Wait();
	}

	if (ReturnCode != -1)
	{
		if (ReturnCodePtr != NULL)
		{
			*ReturnCodePtr = ReturnCode;
		}
		return true;
	}

	return false;
}

void FProcState::Wait()
{
	if (bHasBeenWaitedFor)
	{
		return;	// we could try waitpid() another time, but why
	}

	for(;;)	// infinite loop in case we get EINTR and have to repeat
	{
		siginfo_t SignalInfo;
		if (waitid(P_PID, GetProcessId(), &SignalInfo, WEXITED | WNOWAIT))
		{
			if (errno != EINTR)
			{
				int ErrNo = errno;
				UE_LOGF(LogHAL, Fatal, "FMacPlatformProcess::WaitForProc: waitid for pid %d failed (errno=%d, %ls)",
					static_cast< int32 >(GetProcessId()), ErrNo, UTF8_TO_TCHAR(strerror(ErrNo)));
				break;	// exit the loop if for some reason Fatal log (above) returns
			}
		}
		else
		{
			check(SignalInfo.si_pid == GetProcessId());
			UE::Mac::AddToProcessReaper(ProcessId);

			ReturnCode = (SignalInfo.si_code == CLD_EXITED) ? SignalInfo.si_status : -1;
			bHasBeenWaitedFor = true;
			bIsRunning = false;	// set in advance
			UE_LOGF(LogHAL, Verbose, "Child %d's return code is %d.", GetProcessId(), ReturnCode);
			break;
		}
	}
}

bool FMacPlatformProcess::IsProcRunning( FProcHandle& ProcessHandle )
{
	bool bIsRunning = false;
	FProcState * ProcInfo = ProcessHandle.GetProcessInfo();

	if (ProcInfo)
	{
		bIsRunning = ProcInfo->IsRunning();
	}
	else if (ProcessHandle.Get() != -1)
	{
		// Process opened with OpenProcess() call (we only have pid)
		int KillResult = kill(ProcessHandle.Get(), 0);	// no actual signal is sent
		check(KillResult != -1 || errno != EINVAL);

		// errno == EPERM: don't have permissions to send signal
		// errno == ESRCH: proc doesn't exist
		bIsRunning = (KillResult == 0);
	}

	return bIsRunning;
}

void FMacPlatformProcess::WaitForProc( FProcHandle& ProcessHandle )
{
	FProcState* ProcInfo = ProcessHandle.GetProcessInfo();
	if (ProcInfo)
	{
		ProcInfo->Wait();
	}
	else if (ProcessHandle.Get() != -1)
	{
		STUBBED("FMacPlatformProcess::WaitForProc() : Waiting on OpenProcess() handle not implemented yet");
	}
}

void FMacPlatformProcess::CloseProc( FProcHandle& ProcessHandle )
{
	// dispose of both handle and process info
	FProcState* ProcInfo = ProcessHandle.GetProcessInfo();
	ProcessHandle.Reset();

	delete ProcInfo;
}

void FMacPlatformProcess::TerminateProc( FProcHandle& ProcessHandle, bool KillTree )
{
	FProcState* ProcInfo = ProcessHandle.GetProcessInfo();
	if (ProcInfo)
	{
		const int32 ProcessID = ProcInfo->GetProcessId();

		if (KillTree)
		{
			FProcEnumerator ProcEnumerator;

			while (ProcEnumerator.MoveNext())
			{
				auto Current = ProcEnumerator.GetCurrent();
				if (Current.GetParentPID() == ProcessID)
				{
					kill(Current.GetPID(), SIGTERM);
				}
			}
		}

		int KillResult = kill(ProcessID, SIGTERM);	// graceful
		check(KillResult != -1 || errno != EINVAL);
	}
	else if (ProcessHandle.Get() != -1)
	{
		STUBBED("FMacPlatformProcess::TerminateProc() : Terminating OpenProcess() handle not implemented");
	}
}

uint32 FMacPlatformProcess::GetCurrentProcessId()
{
	return getpid();
}

uint32 FMacPlatformProcess::GetCurrentCoreNumber()
{
#if PLATFORM_MAC_X86
	int CPUInfo[4];
	__cpuid(1, CPUInfo[0], CPUInfo[1], CPUInfo[2], CPUInfo[3]);
	return (CPUInfo[1] >> 24) & 0xff;
#else
    // MAC_ARM TODO - not implemented on iOS either.
    return 0;
#endif
}

bool FMacPlatformProcess::GetProcReturnCode( FProcHandle& ProcessHandle, int32* ReturnCode )
{
	if (IsProcRunning(ProcessHandle))
	{
		return false;
	}

	FProcState* ProcInfo = ProcessHandle.GetProcessInfo();
	if (ProcInfo)
	{
		return ProcInfo->GetReturnCode(ReturnCode);
	}
	else if (ProcessHandle.Get() != -1)
	{
		STUBBED("FMacPlatformProcess::GetProcReturnCode() : Return code of OpenProcess() handle not implemented yet");
	}

	return false;
}

bool FMacPlatformProcess::IsApplicationRunning( uint32 ProcessId )
{
	// PID 0 is not a valid user application so lets ignore it as valid
	if (ProcessId == 0)
	{
		return false;
	}

	errno = 0;
	getpriority(PRIO_PROCESS, ProcessId);
	return errno == 0;
}

FString FMacPlatformProcess::GetApplicationName( uint32 ProcessId )
{
	FString Output;

	char Buffer[MAC_MAX_PATH];
	int32 Ret = proc_pidpath(ProcessId, Buffer, sizeof(Buffer));
	if (Ret > 0)
	{
		Output = ANSI_TO_TCHAR(Buffer);
	}
	return Output;
}

bool FMacPlatformProcess::IsSandboxedApplication()
{
	static const bool bIsSandboxedApplication = []
	{
		SCOPED_AUTORELEASE_POOL;

		SecStaticCodeRef SecCodeObj = nullptr;
		NSURL* BundleURL = [[NSBundle mainBundle] bundleURL];
		OSStatus Err = SecStaticCodeCreateWithPath((CFURLRef)BundleURL, kSecCSDefaultFlags, &SecCodeObj);
		if (SecCodeObj == nullptr)
		{
			return false;
		}

		check(Err == errSecSuccess);

		SecRequirementRef SandboxRequirement = nullptr;
		Err = SecRequirementCreateWithString(CFSTR("entitlement[\"com.apple.security.app-sandbox\"] exists"), kSecCSDefaultFlags, &SandboxRequirement);
		check(Err == errSecSuccess && SandboxRequirement);

		Err = SecStaticCodeCheckValidityWithErrors(SecCodeObj, kSecCSBasicValidateOnly, SandboxRequirement, nullptr);

		if (SandboxRequirement)
		{
			CFRelease(SandboxRequirement);
		}
		CFRelease(SecCodeObj);

		return (Err == errSecSuccess);
	}();

	return bIsSandboxedApplication;
}

const TCHAR* FMacPlatformProcess::BaseDir()
{
	static TCHAR Result[MAC_MAX_PATH] = TEXT("");
	if (!Result[0])
	{
		SCOPED_AUTORELEASE_POOL;
		
		FString CommandLine = [[[NSProcessInfo processInfo] arguments] componentsJoinedByString:@" "];
	
		FString BaseArg;
		FParse::Value(*CommandLine, TEXT("-basedir="), BaseArg);

		if (BaseArg.Len())
		{
			BaseArg = BaseArg.Replace(TEXT("\\"), TEXT("/"));
			BaseArg += TEXT('/');
			FCString::Strcpy(Result, *BaseArg);
			
			FPlatformMisc::LowLevelOutputDebugStringf(TEXT("BaseDir set to %s"), Result);
		}
		else
		{
			NSFileManager* FileManager = [NSFileManager defaultManager];
			NSString *BasePath = [[NSBundle mainBundle] bundlePath];
			// If it has .app extension, it's a bundle, otherwise BasePath is a full path to Binaries/Mac (in case of command line tools)
			if ([[BasePath pathExtension] isEqual: @"app"])
			{
				NSString* BundledBinariesPath = NULL;
				if (!FApp::IsProjectNameEmpty())
				{
					BundledBinariesPath = [BasePath stringByAppendingPathComponent : [NSString stringWithFormat : @"Contents/UE/%s/Binaries/Mac", TCHAR_TO_UTF8(FApp::GetProjectName())]];
				}
				if (!BundledBinariesPath || ![FileManager fileExistsAtPath:BundledBinariesPath])
				{
					BundledBinariesPath = [BasePath stringByAppendingPathComponent: @"Contents/UE/Engine/Binaries/Mac"];
				}
				if ([FileManager fileExistsAtPath: BundledBinariesPath])
				{
					BasePath = BundledBinariesPath;
				}
				else
				{
					BasePath = [BasePath stringByDeletingLastPathComponent];
				}
			}
#ifdef UE_RELATIVE_BASE_DIR
			else
			{
				// Get executable path
				NSString* ExecutablePath = [[NSBundle mainBundle] executablePath];

				// When building an extra console app i.e. a commandlet, the Mac executable can  be placed in a distinct folder, usually 3 directories up compared to the packaged app
				if ([[ExecutablePath lowercaseString] hasSuffix : @"-cmd"] )
				{
#ifdef UE_CMDLET_RELATIVE_BASE_DIR
					BasePath = [BasePath stringByAppendingPathComponent : @UE_CMDLET_RELATIVE_BASE_DIR];
#endif
				}
				else
				{
					BasePath = [BasePath stringByAppendingPathComponent : @UE_RELATIVE_BASE_DIR];
				}
			}
#endif

			FCString::Strncpy(Result, *FString(BasePath), MAC_MAX_PATH);
			FCString::StrncatTruncateDest(Result, MAC_MAX_PATH, TEXT("/"));
		}
	}
	return Result;
}

static TCHAR* UserLibrarySubDirectory()
{
	static TCHAR Result[MAC_MAX_PATH] = TEXT("");
	if (!Result[0])
	{
		FString SubDirectory = IsRunningGame() ? FString(FApp::GetProjectName()) : FString("Unreal Engine") / FApp::GetProjectName();
		if (IsRunningDedicatedServer())
		{
			SubDirectory += TEXT("Server");
		}
		else if (!IsRunningGame())
		{
#if WITH_EDITOR
			SubDirectory += TEXT("Editor");
#endif
		}
		SubDirectory += TEXT("/");
		FCString::Strcpy(Result, *SubDirectory);
	}
	return Result;
}

const TCHAR* FMacPlatformProcess::UserPreferencesDir()
{
	static TCHAR Result[MAC_MAX_PATH] = TEXT("");
	if (!Result[0])
	{
		SCOPED_AUTORELEASE_POOL;
		NSString *UserLibraryDirectory = [NSSearchPathForDirectoriesInDomains(NSLibraryDirectory, NSUserDomainMask, YES) objectAtIndex: 0];
		FPlatformString::CFStringToTCHAR((CFStringRef)UserLibraryDirectory, Result);
		FCString::Strcat(Result, TEXT("/Preferences/"));
		FCString::Strcat(Result, UserLibrarySubDirectory());
	}
	return Result;
}

const TCHAR* FMacPlatformProcess::UserLogsDir()
{
	static TCHAR Result[MAC_MAX_PATH] = TEXT("");
	if (!Result[0])
	{
		SCOPED_AUTORELEASE_POOL;
		NSString *UserLibraryDirectory = [NSSearchPathForDirectoriesInDomains(NSLibraryDirectory, NSUserDomainMask, YES) objectAtIndex: 0];
		FPlatformString::CFStringToTCHAR((CFStringRef)UserLibraryDirectory, Result);
		FCString::Strcat(Result, TEXT("/Logs/"));
		FCString::Strcat(Result, UserLibrarySubDirectory());
	}
	return Result;
}

const TCHAR* FMacPlatformProcess::ComputerName()
{
	static TCHAR Result[256]=TEXT("");

	if( !Result[0] )
	{
		ANSICHAR AnsiResult[UE_ARRAY_COUNT(Result)];
		gethostname(AnsiResult, UE_ARRAY_COUNT(Result));
		FCString::Strcpy(Result, ANSI_TO_TCHAR(AnsiResult));
	}
	return Result;
}

const TCHAR* FMacPlatformProcess::UserName(bool bOnlyAlphaNumeric)
{
	static TCHAR Result[256]=TEXT("");
	static TCHAR ResultAlpha[256]=TEXT("");
	if( bOnlyAlphaNumeric )
	{
		if( !ResultAlpha[0] )
		{
			SCOPED_AUTORELEASE_POOL;
			FPlatformString::CFStringToTCHAR( ( CFStringRef )NSUserName(), ResultAlpha );
			TCHAR *c, *d;
			for( c=ResultAlpha, d=ResultAlpha; *c!=0; c++ )
				if( FChar::IsAlnum(*c) )
					*d++ = *c;
			*d++ = 0;
		}
		return ResultAlpha;
	}
	else
	{
		if( !Result[0] )
		{
			SCOPED_AUTORELEASE_POOL;
			FPlatformString::CFStringToTCHAR( ( CFStringRef )NSUserName(), Result );
		}
		return Result;
	}
}

void FMacPlatformProcess::SetCurrentWorkingDirectoryToBaseDir()
{
#if defined(DISABLE_CWD_CHANGES) && DISABLE_CWD_CHANGES != 0
	check(false);
#else
	FPlatformMisc::CacheLaunchDir();
	chdir([FString(BaseDir()).GetNSString() fileSystemRepresentation]);
#endif
}

FString FMacPlatformProcess::GetCurrentWorkingDirectory()
{
	// get the current directory
	ANSICHAR CurrentDir[MAC_MAX_PATH] = { 0 };
	getcwd(CurrentDir, sizeof(CurrentDir));
	return UTF8_TO_TCHAR(CurrentDir);
}

const TCHAR* FMacPlatformProcess::ExecutablePath()
{
	static TCHAR Result[512]=TEXT("");
	if( !Result[0] )
	{
		SCOPED_AUTORELEASE_POOL;
		NSString *NSExeName = [[NSBundle mainBundle] executablePath];
		FPlatformString::CFStringToTCHAR( ( CFStringRef )NSExeName, Result );
	}
	return Result;
}

const TCHAR* FMacPlatformProcess::ExecutableName(bool bRemoveExtension)
{
	static TCHAR Result[512]=TEXT("");
	if( !Result[0] )
	{
		SCOPED_AUTORELEASE_POOL;
		NSString *NSExeName = [[[NSBundle mainBundle] executablePath] lastPathComponent];
		FPlatformString::CFStringToTCHAR( ( CFStringRef )NSExeName, Result );
	}
	return Result;
}

const TCHAR* FMacPlatformProcess::GetModuleExtension()
{
	return TEXT("dylib");
}

const TCHAR* FMacPlatformProcess::GetBinariesSubdirectory()
{
	return TEXT("Mac");
}

const FString FMacPlatformProcess::GetModulesDirectory()
{
	return FString(BaseDir());
}

bool FMacPlatformProcess::LaunchFileInDefaultExternalApplication( const TCHAR* FileName, const TCHAR* Parms /*= NULL*/, ELaunchVerb::Type Verb /*= ELaunchVerb::Open*/, bool bPromptToOpenOnFailure /*= true */)
{
	SCOPED_AUTORELEASE_POOL;
	// First attempt to open the file in its default application
	UE_LOGF(LogMac, Log,  "LaunchFileInExternalEditor %ls %ls", FileName, Parms ? Parms : TEXT("") );
	CFStringRef CFFileName = FPlatformString::TCHARToCFString( FileName );
	NSString* FileToOpen = ( NSString* )CFFileName;
	if( [[FileToOpen lastPathComponent] isEqualToString: @"project.pbxproj"] || [[FileToOpen lastPathComponent] isEqualToString: @"contents.xcworkspacedata"] )
	{
		// Xcode project is a special case where we don't open the project file itself, but the .xcodeproj folder containing it
		FileToOpen = [FileToOpen stringByDeletingLastPathComponent];
	}
	bool Result = [[NSWorkspace sharedWorkspace] openURL: [NSURL fileURLWithPath:FileToOpen]];
	CFRelease( CFFileName );

	return Result;
}

void FMacPlatformProcess::ExploreFolder( const TCHAR* FilePath )
{
    extern void MainThreadCall(dispatch_block_t Block, bool const bWait, NSString* WaitMode);

	SCOPED_AUTORELEASE_POOL;
	__block CFStringRef CFFilePath = FPlatformString::TCHARToCFString( FilePath );

    MainThreadCall(^{
	BOOL IsDirectory = NO;
	if([[NSFileManager defaultManager] fileExistsAtPath:(NSString *)CFFilePath isDirectory:&IsDirectory])
	{
		if(IsDirectory)
		{
			[[NSWorkspace sharedWorkspace] selectFile:nil inFileViewerRootedAtPath:(NSString *)CFFilePath];
		}
		else
		{
			NSString* Directory = [(NSString *)CFFilePath stringByDeletingLastPathComponent];
			[[NSWorkspace sharedWorkspace] selectFile:(NSString *)CFFilePath inFileViewerRootedAtPath:Directory];
		}
	}
	CFRelease( CFFilePath );
    }, false, nullptr);
}

void FMacPlatformProcess::ClosePipe( void* ReadPipe, void* WritePipe )
{
	SCOPED_AUTORELEASE_POOL;
	if(ReadPipe)
	{
		close([(NSFileHandle*)ReadPipe fileDescriptor]);
		[(NSFileHandle*)ReadPipe release];
	}
	if(WritePipe)
	{
		close([(NSFileHandle*)WritePipe fileDescriptor]);
		[(NSFileHandle*)WritePipe release];
	}
}

bool FMacPlatformProcess::CreatePipe( void*& ReadPipe, void*& WritePipe, bool bWritePipeLocal )
{
	SCOPED_AUTORELEASE_POOL;
	int pipefd[2];
	pipe(pipefd);

	if (bWritePipeLocal)
	{
		fcntl(pipefd[1], F_SETFL, O_CLOEXEC);
	}
	else
	{
		// The read pipe should be non-blocking, but the write pipe SHOULD block
		fcntl(pipefd[0], F_SETFL, O_CLOEXEC | O_NONBLOCK);
	}

	// create an NSFileHandle from the descriptor
	ReadPipe = [[NSFileHandle alloc] initWithFileDescriptor: pipefd[0]];
	WritePipe = [[NSFileHandle alloc] initWithFileDescriptor: pipefd[1]];

	return true;
}

FString FMacPlatformProcess::ReadPipe( void* ReadPipe )
{
	SCOPED_AUTORELEASE_POOL;

	const int32 READ_SIZE = 8192;
	ANSICHAR Buffer[READ_SIZE+1];
	int32 BytesRead = 0;

	// We don't want to use FUtf8StringBuilderBase here because the Buffer boundary could split a UTF-8 character
	// and we don't want .Append to attempt to interpert the incoming data.  We will convert the whole string in one go at the end.
	TAnsiStringBuilder<0> StringBuilder;

	if (ReadPipe)
	{
		do
		{
			BytesRead = read([(NSFileHandle*)ReadPipe fileDescriptor], Buffer, READ_SIZE);
			if (BytesRead > 0)
			{
				Buffer[BytesRead] = '\0';
				StringBuilder.Append(Buffer, BytesRead);
			}
		} while (BytesRead > 0);
	}
	
	return FString((const UTF8CHAR*)*StringBuilder);
}

bool FMacPlatformProcess::ReadPipeToArray(void* ReadPipe, TArray<uint8>& Output)
{
	SCOPED_AUTORELEASE_POOL;

	const int32 READ_SIZE = 32768;

	if (ReadPipe)
	{
		Output.SetNumUninitialized(READ_SIZE);
		int32 BytesRead = 0;
		BytesRead = read([(NSFileHandle*)ReadPipe fileDescriptor], Output.GetData(), READ_SIZE);
		if (BytesRead > 0)
		{
			if (BytesRead < READ_SIZE)
			{
				Output.SetNum(BytesRead);
			}

			return true;
		}
		else
		{
			Output.Empty();
		}
	}

	return false;
}

bool FMacPlatformProcess::WritePipe(void* WritePipe, const FString& Message, FString* OutWritten)
{
	// if there is not a message or WritePipe is nullptr
	int32 MessageLen = Message.Len();
	if ((MessageLen == 0) || (WritePipe == nullptr))
	{
		return false;
	}

	// Convert input to UTF8CHAR
	const TCHAR* MessagePtr = *Message;
	int32 BytesAvailable = FPlatformString::ConvertedLength<UTF8CHAR>(MessagePtr, MessageLen);
	UTF8CHAR* Buffer = new UTF8CHAR[BytesAvailable + 2];
	*FPlatformString::Convert(Buffer, BytesAvailable, MessagePtr, MessageLen) = (UTF8CHAR)'\n';

	// Write to pipe
	uint32 BytesWritten = write([(NSFileHandle*)WritePipe fileDescriptor], Buffer, BytesAvailable + 1);

	// Get written message
	if (OutWritten)
	{
		*OutWritten = StringCast<TCHAR>(Buffer, BytesWritten).Get();
	}

	delete[] Buffer;
	return (BytesWritten == BytesAvailable);
}

bool FMacPlatformProcess::WritePipe(void* WritePipe, const uint8* Data, const int32 DataLength, int32* OutDataLength)
{
	// if there is not a message or WritePipe is null
	if ((DataLength == 0) || (WritePipe == nullptr))
	{
		return false;
	}

	// write to pipe
	uint32 BytesWritten = write([(NSFileHandle*)WritePipe fileDescriptor], Data, DataLength);

	// Get written Data Length
	if (OutDataLength)
	{
		*OutDataLength = (int32)BytesWritten;
	}

	return (BytesWritten == DataLength);
}

bool FMacPlatformProcess::IsApplicationRunning(const TCHAR* ProcName)
{
	const FString ProcString = FPaths::GetCleanFilename(ProcName);
	uint32 ThisProcessID = getpid();

	FProcEnumerator ProcEnumerator;

	while (ProcEnumerator.MoveNext())
	{
		auto Current = ProcEnumerator.GetCurrent();
		if (Current.GetPID() != ThisProcessID && Current.GetName() == ProcString)
		{
			return true;
		}
	}

	return false;
}

FMacPlatformProcess::FProcEnumerator::FProcEnumerator()
{
	int32 Mib[4] = { CTL_KERN, KERN_PROC, KERN_PROC_ALL, 0 };
	size_t BufferSize = 0;

	Processes = nullptr;
	ProcCount = 0;
	CurrentProcIndex = -1;

	if (sysctl(Mib, 4, NULL, &BufferSize, NULL, 0) != -1 && BufferSize > 0)
	{
		char Buffer[MAC_MAX_PATH];
		Processes = (struct kinfo_proc*)FMemory::Malloc(BufferSize);
		if (sysctl(Mib, 4, Processes, &BufferSize, NULL, 0) != -1)
		{
			ProcCount = (uint32)(BufferSize / sizeof(struct kinfo_proc));
		}
	}
}

FMacPlatformProcess::FProcEnumerator::~FProcEnumerator()
{
	if (Processes != nullptr)
	{
		FMemory::Free(Processes);
	}
}

FMacPlatformProcess::FProcEnumInfo::FProcEnumInfo(struct kinfo_proc InProcInfo)
	: ProcInfo(InProcInfo)
{

}

bool FMacPlatformProcess::FProcEnumerator::MoveNext()
{
	if (CurrentProcIndex == ProcCount)
	{
		return false;
	}

	++CurrentProcIndex;
	return true;
}

FMacPlatformProcess::FProcEnumInfo FMacPlatformProcess::FProcEnumerator::GetCurrent() const
{
	return FProcEnumInfo(Processes[CurrentProcIndex]);
}

uint32 FMacPlatformProcess::FProcEnumInfo::GetPID() const
{
	return ProcInfo.kp_proc.p_pid;
}

uint32 FMacPlatformProcess::FProcEnumInfo::GetParentPID() const
{
	return ProcInfo.kp_eproc.e_ppid;
}

FString FMacPlatformProcess::FProcEnumInfo::GetFullPath() const
{
	char Buffer[MAC_MAX_PATH];
	proc_pidpath(GetPID(), Buffer, sizeof(Buffer));

	return Buffer;
}

FString FMacPlatformProcess::FProcEnumInfo::GetName() const
{
	return FPaths::GetCleanFilename(GetFullPath());
}

FRunnableThread* FMacPlatformProcess::CreateRunnableThread()
{
	return new FRunnableThreadApple();
}

void FMacPlatformProcess::SetThreadAffinityMask(uint64 AffinityMask)
{
	if( AffinityMask != FPlatformAffinity::GetNoAffinityMask() )
	{
		thread_affinity_policy AP;
		AP.affinity_tag = AffinityMask;
		thread_policy_set(pthread_mach_thread_np(pthread_self()), THREAD_AFFINITY_POLICY, (integer_t*)&AP, THREAD_AFFINITY_POLICY_COUNT);
	}
}
