// Copyright Epic Games, Inc. All Rights Reserved.

#include "Mac/MacSemaphore.h"
#include "Misc/AssertionMacros.h"
#include <dispatch/dispatch.h>

FMacSemaphore::FMacSemaphore(int32 InitialCount, int32 /*MaxCount*/)
	: Semaphore((dispatch_semaphore_t)dispatch_semaphore_create(InitialCount))
{
	checkfSlow(InitialCount >= 0, TEXT("Semaphore's initial count must be non negative value: %d"), InitialCount);
}

FMacSemaphore::~FMacSemaphore()
{
	dispatch_release((dispatch_semaphore_t)Semaphore);
}

void FMacSemaphore::Acquire()
{
	intptr_t Res = dispatch_semaphore_wait((dispatch_semaphore_t)Semaphore, DISPATCH_TIME_FOREVER);
	checkfSlow(Res == 0, TEXT("Acquiring semaphore failed"));
}

bool FMacSemaphore::TryAcquire(FTimespan Timeout)
{
	dispatch_time_t TS = dispatch_time(DISPATCH_TIME_NOW, (int64)Timeout.GetTotalMicroseconds() * 1000);
	intptr_t Res = dispatch_semaphore_wait((dispatch_semaphore_t)Semaphore, TS);
	return Res == 0;
}

void FMacSemaphore::Release(int32 Count)
{
	checkfSlow(Count > 0, TEXT("Releasing semaphore with Count = %d, that should be greater than 0"), Count);
	for (int i = 0; i != Count; ++i)
	{
		dispatch_semaphore_signal((dispatch_semaphore_t)Semaphore);
	}
}
