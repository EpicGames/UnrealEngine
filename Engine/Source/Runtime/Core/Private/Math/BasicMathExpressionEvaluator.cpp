// Copyright Epic Games, Inc. All Rights Reserved.

#include "Math/BasicMathExpressionEvaluator.h"
#include "Misc/AutomationTest.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/ExpressionParser.h"
#include "Internationalization/Culture.h"
#include "Internationalization/Internationalization.h"

#define LOCTEXT_NAMESPACE "BasicMathExpressionEvaluator"

namespace ExpressionParser
{
	const TCHAR* const FSubExpressionStart::Moniker = TEXT("(");
	const TCHAR* const FSubExpressionEnd::Moniker = TEXT(")");
	const TCHAR* const FPlus::Moniker = TEXT("+");
	const TCHAR* const FPlusEquals::Moniker = TEXT("+=");
	const TCHAR* const FMinus::Moniker = TEXT("-");
	const TCHAR* const FMinusEquals::Moniker = TEXT("-=");
	const TCHAR* const FStar::Moniker = TEXT("*");
	const TCHAR* const FStarEquals::Moniker = TEXT("*=");
	const TCHAR* const FForwardSlash::Moniker = TEXT("/");
	const TCHAR* const FForwardSlashEquals::Moniker = TEXT("/=");
	const TCHAR* const FPercent::Moniker = TEXT("%");
	const TCHAR* const FSquareRoot::Moniker = TEXT("sqrt");
	const TCHAR* const FPower::Moniker = TEXT("^");

	const FDecimalNumberFormattingRules& GetLocalizedNumberFormattingRules()
	{
		bool bShouldUseLocalizedNumericInput = false;
		GConfig->GetBool(TEXT("Internationalization"), TEXT("ShouldUseLocalizedNumericInput"), bShouldUseLocalizedNumericInput, GIsEditor ? GEditorSettingsIni : GGameUserSettingsIni);
		return bShouldUseLocalizedNumericInput 
			? FInternationalization::Get().GetCurrentLocale()->GetDecimalNumberFormattingRules() 
			: FastDecimalFormat::GetCultureAgnosticFormattingRules();
	}

	TOptional<FStringToken> ParseNumberWithFallback(const FTokenStream& InStream, const FDecimalNumberFormattingRules& InPrimaryFormattingRules, const FDecimalNumberFormattingRules& InFallbackFormattingRules, FStringToken* Accumulate, double* OutValue)
	{
		// Attempt to parse a number from the string
		// This call will return false if there is some other data after the number, which is why we check the parsed length instead
		double PrimaryValue = 0.0;
		int32 PrimaryParsedLen = 0;
		FastDecimalFormat::StringToNumber(InStream.GetRead(), UE_PTRDIFF_TO_INT32(InStream.GetEnd() - InStream.GetRead()), InPrimaryFormattingRules, FNumberParsingOptions::DefaultNoGrouping(), PrimaryValue, &PrimaryParsedLen);

		double FallbackValue = 0.0;
		int32 FallbackParsedLen = 0;
		FastDecimalFormat::StringToNumber(InStream.GetRead(), UE_PTRDIFF_TO_INT32(InStream.GetEnd() - InStream.GetRead()), InFallbackFormattingRules, FNumberParsingOptions::DefaultNoGrouping(), FallbackValue, &FallbackParsedLen);

		// We take whichever value parsed the most text from the string
		if (FallbackParsedLen <= PrimaryParsedLen)
		{
			if (OutValue)
			{
				*OutValue = PrimaryValue;
			}

			return PrimaryParsedLen > 0 ? InStream.GenerateToken(PrimaryParsedLen) : TOptional<FStringToken>();
		}
		else
		{
			if (OutValue)
			{
				*OutValue = FallbackValue;
			}

			return FallbackParsedLen > 0 ? InStream.GenerateToken(FallbackParsedLen) : TOptional<FStringToken>();
		}
	}

	TOptional<FStringToken> ParseNumberWithRules(const FTokenStream& InStream, const FDecimalNumberFormattingRules& InFormattingRules, FStringToken* Accumulate, double* OutValue)
	{
		// Attempt to parse a number from the string
		// This call will return false if there is some other data after the number, which is why we check the parsed length instead
		double Value = 0.0;
		int32 ParsedLen = 0;
		FastDecimalFormat::StringToNumber(InStream.GetRead(), UE_PTRDIFF_TO_INT32(InStream.GetEnd() - InStream.GetRead()), InFormattingRules, FNumberParsingOptions::DefaultNoGrouping(), Value, &ParsedLen);

		if (OutValue)
		{
			*OutValue = Value;
		}

		return ParsedLen > 0 ? InStream.GenerateToken(ParsedLen) : TOptional<FStringToken>();
	}

	TOptional<FStringToken> ParseLocalizedNumberWithAgnosticFallback(const FTokenStream& InStream, FStringToken* Accumulate, double* OutValue)
	{
		return ParseNumberWithFallback(InStream, GetLocalizedNumberFormattingRules(), FastDecimalFormat::GetCultureAgnosticFormattingRules(), Accumulate, OutValue);
	}

	TOptional<FStringToken> ParseLocalizedNumber(const FTokenStream& InStream, FStringToken* Accumulate, double* OutValue)
	{
		return ParseNumberWithRules(InStream, GetLocalizedNumberFormattingRules(), Accumulate, OutValue);
	}

	TOptional<FStringToken> ParseNumber(const FTokenStream& InStream, FStringToken* Accumulate, double* OutValue)
	{
		return ParseNumberWithRules(InStream, FastDecimalFormat::GetCultureAgnosticFormattingRules(), Accumulate, OutValue);
	}

	TOptional<FExpressionError> ConsumeNumberWithRules(FExpressionTokenConsumer& Consumer, const FDecimalNumberFormattingRules& InFormattingRules)
	{
		auto& Stream = Consumer.GetStream();

		double Value = 0.0;
		TOptional<FStringToken> Token = ParseNumberWithRules(Stream, InFormattingRules, nullptr, &Value);

		if (Token.IsSet())
		{
			Consumer.Add(Token.GetValue(), FExpressionNode(Value));
		}

		return TOptional<FExpressionError>();
	}

	TOptional<FExpressionError> ConsumeLocalizedNumberWithAgnosticFallback(FExpressionTokenConsumer& Consumer)
	{
		auto& Stream = Consumer.GetStream();

		double Value = 0.0;
		TOptional<FStringToken> Token = ParseLocalizedNumberWithAgnosticFallback(Stream, nullptr, &Value);

		if (Token.IsSet())
		{
			Consumer.Add(Token.GetValue(), FExpressionNode(Value));
		}

		return TOptional<FExpressionError>();
	}

	TOptional<FExpressionError> ConsumeLocalizedNumber(FExpressionTokenConsumer& Consumer)
	{
		return ConsumeNumberWithRules(Consumer, GetLocalizedNumberFormattingRules());
	}

	TOptional<FExpressionError> ConsumeNumber(FExpressionTokenConsumer& Consumer)
	{
		return ConsumeNumberWithRules(Consumer, FastDecimalFormat::GetCultureAgnosticFormattingRules());
	}
}

void FBasicMathExpressionEvaluator::DefineArithmeticTokens(FTokenDefinitions& TokenDefinitions)
{
	using namespace ExpressionParser;

	TokenDefinitions.DefineToken(&ConsumeSymbol<FPlusEquals>);
	TokenDefinitions.DefineToken(&ConsumeSymbol<FMinusEquals>);
	TokenDefinitions.DefineToken(&ConsumeSymbol<FStarEquals>);
	TokenDefinitions.DefineToken(&ConsumeSymbol<FForwardSlashEquals>);
	TokenDefinitions.DefineToken(&ConsumeSymbol<FPlus>);
	TokenDefinitions.DefineToken(&ConsumeSymbol<FMinus>);
	TokenDefinitions.DefineToken(&ConsumeSymbol<FStar>);
	TokenDefinitions.DefineToken(&ConsumeSymbol<FForwardSlash>);
	TokenDefinitions.DefineToken(&ConsumeSymbol<FPercent>);
	TokenDefinitions.DefineToken(&ConsumeSymbol<FSquareRoot>);
	TokenDefinitions.DefineToken(&ConsumeSymbol<FPower>);
}

void FBasicMathExpressionEvaluator::DefineArithmeticOperators(FExpressionGrammar& Grammar)
{
	using namespace ExpressionParser;

	Grammar.DefinePreUnaryOperator<FPlus>();
	Grammar.DefinePreUnaryOperator<FMinus>();
	Grammar.DefinePreUnaryOperator<FSquareRoot>();

	// Left-to-right evaluation is required for non-commutative binary operations, and a reasonable default for commutative ones too.
	Grammar.DefineBinaryOperator<FPlus>(5, EAssociativity::LeftToRight);
	Grammar.DefineBinaryOperator<FMinus>(5, EAssociativity::LeftToRight);
	Grammar.DefineBinaryOperator<FStar>(4, EAssociativity::LeftToRight);
	Grammar.DefineBinaryOperator<FForwardSlash>(4, EAssociativity::LeftToRight);
	Grammar.DefineBinaryOperator<FPercent>(4, EAssociativity::LeftToRight);
	Grammar.DefineBinaryOperator<FPower>(3);

}

void FBasicMathExpressionEvaluator::MapArithmeticOperators(FOperatorJumpTable& JumpTable)
{
	using namespace ExpressionParser;

	JumpTable.MapPreUnary<FPlus>([](double N)			{ return N; });
	JumpTable.MapPreUnary<FMinus>([](double N)			{ return -N; });
	JumpTable.MapPreUnary<FSquareRoot>([](double A)		{ return double(FMath::Sqrt((float)A)); }); //@TODO: FLOATPRECISION: Needs a double version of Sqrt

	JumpTable.MapBinary<FPlus>([](double A, double B)	{ return A + B; });
	JumpTable.MapBinary<FMinus>([](double A, double B)	{ return A - B; });
	JumpTable.MapBinary<FStar>([](double A, double B)	{ return A * B; });
	JumpTable.MapBinary<FPower>([](double A, double B)	{ return double(FMath::Pow((float)A, (float)B)); }); //@TODO: FLOATPRECISION: Needs a double version of Pow

	JumpTable.MapBinary<FForwardSlash>([](double A, double B) -> FExpressionResult {
		if (B == 0)
		{
			return MakeError(LOCTEXT("DivisionByZero", "Division by zero"));
		}

		return MakeValue(A / B);
	});
	JumpTable.MapBinary<FPercent>([](double A, double B) -> FExpressionResult {
		if (B == 0)
		{
			return MakeError(LOCTEXT("ModZero", "Modulo zero"));
		}

		return MakeValue(double(FMath::Fmod((float)A, (float)B))); //@TODO: FLOATPRECISION: Needs a double version of FMod
	});
}

FBasicMathExpressionEvaluator::FBasicMathExpressionEvaluator()
{
	using namespace ExpressionParser;

	TokenDefinitions.IgnoreWhitespace();
	TokenDefinitions.DefineToken(&ConsumeSymbol<FSubExpressionStart>);
	TokenDefinitions.DefineToken(&ConsumeSymbol<FSubExpressionEnd>);
	DefineArithmeticTokens(TokenDefinitions);
	TokenDefinitions.DefineToken(&ConsumeLocalizedNumberWithAgnosticFallback);

	Grammar.DefineGrouping<FSubExpressionStart, FSubExpressionEnd>();
	DefineArithmeticOperators(Grammar);

	MapArithmeticOperators(JumpTable);
}

TValueOrError<double, FExpressionError> FBasicMathExpressionEvaluator::Evaluate(const TCHAR* InExpression, double InExistingValue) const
{
	using namespace ExpressionParser;

	TValueOrError<TArray<FExpressionToken>, FExpressionError> LexResult = ExpressionParser::Lex(InExpression, TokenDefinitions);
	if (!LexResult.IsValid())
	{
		return MakeError(LexResult.StealError());
	}

	// Handle the += and -= tokens.
	TArray<FExpressionToken> Tokens = LexResult.StealValue();
	if (Tokens.Num())
	{
		FStringToken Context = Tokens[0].Context;
		const FExpressionNode& FirstNode = Tokens[0].Node;
		bool WasOpAssign = true;

		if (FirstNode.Cast<FPlusEquals>())
		{
			Tokens.Insert(FExpressionToken(Context, FPlus()), 0);
		}
		else if (FirstNode.Cast<FMinusEquals>())
		{
			Tokens.Insert(FExpressionToken(Context, FMinus()), 0);
		}
		else if (FirstNode.Cast<FStarEquals>())
		{
			Tokens.Insert(FExpressionToken(Context, FStar()), 0);
		}
		else if (FirstNode.Cast<FForwardSlashEquals>())
		{
			Tokens.Insert(FExpressionToken(Context, FForwardSlash()), 0);
		}
		else
		{
			WasOpAssign = false;
		}

		if (WasOpAssign)
		{
			Tokens.Insert(FExpressionToken(Context, InExistingValue), 0);
			Tokens.RemoveAt(2, EAllowShrinking::No);
		}
	}

	TValueOrError<TArray<FCompiledToken>, FExpressionError> CompilationResult = ExpressionParser::Compile(MoveTemp(Tokens), Grammar);
	if (!CompilationResult.IsValid())
	{
		return MakeError(CompilationResult.StealError());
	}

	TOperatorEvaluationEnvironment<> Env(JumpTable, nullptr);
	TValueOrError<FExpressionNode, FExpressionError> Result = ExpressionParser::Evaluate(CompilationResult.GetValue(), Env);
	if (!Result.IsValid())
	{
		return MakeError(Result.GetError());
	}

	auto& Node = Result.GetValue();

	if (const auto* Numeric = Node.Cast<double>())
	{
		return MakeValue(*Numeric);
	}

	return MakeError(LOCTEXT("UnrecognizedResult", "Unrecognized result returned from expression"));
}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

bool TestExpression(FAutomationTestBase* Test, const TCHAR* Expression, double Expected)
{
	FBasicMathExpressionEvaluator Parser;

	TValueOrError<double, FExpressionError> Result = Parser.Evaluate(Expression);
	if (!Result.IsValid())
	{
		Test->AddError(Result.GetError().Text.ToString());
		return false;
	}
	
	if (Result.GetValue() != Expected)
	{
		Test->AddError(FString::Printf(TEXT("'%s' evaluation results: %.f != %.f"), Expression, Result.GetValue(), Expected));
		return false;
	}

	return true;
}

bool TestInvalidExpression(FAutomationTestBase* Test, const TCHAR* Expression)
{
	FBasicMathExpressionEvaluator Parser;

	TValueOrError<double, FExpressionError> Result = Parser.Evaluate(Expression);
	if (Result.IsValid())
	{
		return false;
	}

	return true;
}


IMPLEMENT_SIMPLE_AUTOMATION_TEST(FBasicMathExpressionEvaluatorTest, "System.Core.Math.Evaluate.Valid Expressions", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
// Evaluates valid math expressions.
bool FBasicMathExpressionEvaluatorTest::RunTest(const FString& Parameters)
{
	TestTrue(TEXT("Valid expression, '+1', evaluated incorrectly."), TestExpression(this, TEXT("+1"), 1));
	TestTrue(TEXT("Valid expression, '-20', evaluated incorrectly."), TestExpression(this, TEXT("-20"), -20));
	TestTrue(TEXT("Valid expression, '-+-2', evaluated incorrectly."), TestExpression(this, TEXT("-+-2"), 2));
	TestTrue(TEXT("Valid expression, '1 + 2', evaluated incorrectly."), TestExpression(this, TEXT("1 + 2"), 3));
	TestTrue(TEXT("Valid expression, '1+2*3', evaluated incorrectly."), TestExpression(this, TEXT("1+2*3"), 7));
	TestTrue(TEXT("Valid expression, '1+2*3*4+1', evaluated incorrectly."), TestExpression(this, TEXT("1+2*3*4+1"), 1 + 2 * 3 * 4 + 1));
	TestTrue(TEXT("Valid expression, '1*2+3', evaluated incorrectly."), TestExpression(this, TEXT("1*2+3"), 1 * 2 + 3));
	TestTrue(TEXT("Valid expression, '1+2*3*4+1', evaluated incorrectly."), TestExpression(this, TEXT("1+2*3*4+1"), 1 + 2 * 3 * 4 + 1));
	TestTrue(TEXT("Valid expression, '8-4+3', evaluated incorrectly."), TestExpression(this, TEXT("8-4+3"), 8 - 4 + 3));
	
	TestTrue(TEXT("Valid expression, '2^2', evaluated incorrectly."), TestExpression(this, TEXT("2^2"), 4));
	TestTrue(TEXT("Valid expression, '2^(2*3)', evaluated incorrectly."), TestExpression(this, TEXT("2^(2*3)"), 64));
	TestTrue(TEXT("Valid expression, '2^2*3', evaluated incorrectly."), TestExpression(this, TEXT("2^2*3"), 12));
	TestTrue(TEXT("Valid expression, 'sqrt(4)', evaluated incorrectly."), TestExpression(this, TEXT("sqrt(4)"), 2));
	TestTrue(TEXT("Valid expression, '4*sqrt(4)+10', evaluated incorrectly."), TestExpression(this, TEXT("4*sqrt(4)+10"), 18));
	TestTrue(TEXT("Valid expression, '8%6', evaluated incorrectly."), TestExpression(this, TEXT("8%6"), 2));

	TestTrue(TEXT("Valid expression, '100-20-10-10', evaluated incorrectly."), TestExpression(this, TEXT("100-20-10-10"), 100 - 20 - 10 - 10));
	TestTrue(TEXT("Valid expression, '100-(20-10)-10', evaluated incorrectly."), TestExpression(this, TEXT("100-(20-10)-10"), 100 - (20 - 10) - 10));
	TestTrue(TEXT("Valid expression, '100/2/5', evaluated incorrectly."), TestExpression(this, TEXT("100/2/5"), 100 / 2 / 5));

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FBasicMathExpressionEvaluatorWhitespaceExpressionsTest, "System.Core.Math.Evaluate.Valid Expressions With Whitespaces", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
// Evaluates a valid math expression with leading and trailing white spaces.
bool FBasicMathExpressionEvaluatorWhitespaceExpressionsTest::RunTest(const FString& Parameters)
{
	TestTrue(TEXT("Expression with leading and trailing whitespaces was not evaluated correctly."), TestExpression(this, TEXT(" 1+2 "), 1 + 2));

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FBasicMathExpressionEvaluatorGroupedExpressionsTest, "System.Core.Math.Evaluate.Valid Grouped Expressions", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
// Evaluates valid math expressions that are grouped
bool FBasicMathExpressionEvaluatorGroupedExpressionsTest::RunTest(const FString& Parameters)
{
	TestTrue(TEXT("Valid grouped expression, '(1+2)*3*4+1', evaluated incorrectly."), TestExpression(this, TEXT("(1+2)*3*4+1"), (1 + 2) * 3 * 4 + 1));
	TestTrue(TEXT("Valid grouped expression, '(1+2)*3*(4+1)', evaluated incorrectly."), TestExpression(this, TEXT("(1+2)*3*(4+1)"), (1 + 2) * 3 * (4 + 1)));
	TestTrue(TEXT("Valid grouped expression, '((1+2) / (3+1) + 2) * 3', evaluated incorrectly."), TestExpression(this, TEXT("((1+2) / (3+1) + 2) * 3"), ((1.0 + 2) / (3 + 1) + 2) * 3));
	TestTrue(TEXT("Valid grouped expression, '8 / 2 * (2 + 2)', evaluated incorrectly."), TestExpression(this, TEXT("8 / 2 * (2 + 2)"), 8 / 2 * (2 + 2)));

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FBasicMathExpressionEvaluatorInvalidExpressionTest, "System.Core.Math.Evaluate.Invalid Expressions", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)

// Evaluates invalid expressions.
// Invalid expressions will report errors and not crash.
bool FBasicMathExpressionEvaluatorInvalidExpressionTest::RunTest(const FString& Parameters)
{
	TestTrue(TEXT("The invalid math expression, 'gobbledegook', did not report an error."), TestInvalidExpression(this, TEXT("gobbledegook")));
	TestTrue(TEXT("The invalid math expression, '50**10', did not report an error."), TestInvalidExpression(this, TEXT("50**10")));
	TestTrue(TEXT("The invalid math expression, '*1', did not report an error."), TestInvalidExpression(this, TEXT("*1")));
	TestTrue(TEXT("The invalid math expression, '+', did not report an error."), TestInvalidExpression(this, TEXT("+")));
	TestTrue(TEXT("The invalid math expression, '{+}', did not report an error."), TestInvalidExpression(this, TEXT("+")));

	return true;
}

#endif //!(UE_BUILD_SHIPPING || UE_BUILD_TEST)

#undef LOCTEXT_NAMESPACE
