// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Box2D.cpp: Implements the FBox2D class.
=============================================================================*/

/* FBox2D structors
 *****************************************************************************/

#include "Math/Box2D.h"

