// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Color.cpp: Unreal color implementation.
=============================================================================*/

#include "Math/Color.h"
#include "Math/Vector.h"
#include "Math/Vector4.h"
#include "Math/Float16Color.h"
#include "Math/RandomStream.h"

#include "ColorManagement/ColorSpace.h"
#include "HAL/IConsoleManager.h"

// Common colors.
const FLinearColor FLinearColor::White(1.f,1.f,1.f);
const FLinearColor FLinearColor::Gray(0.5f,0.5f,0.5f);
const FLinearColor FLinearColor::Black(0,0,0);
const FLinearColor FLinearColor::Transparent(0,0,0,0);
const FLinearColor FLinearColor::Red(1.f,0,0);
const FLinearColor FLinearColor::Green(0,1.f,0);
const FLinearColor FLinearColor::Blue(0,0,1.f);
const FLinearColor FLinearColor::Yellow(1.f,1.f,0);

const FColor FColor::White(255,255,255);
const FColor FColor::Black(0,0,0);
const FColor FColor::Transparent(0, 0, 0, 0);
const FColor FColor::Red(255,0,0);
const FColor FColor::Green(0,255,0);
const FColor FColor::Blue(0,0,255);
const FColor FColor::Yellow(255,255,0);
const FColor FColor::Cyan(0,255,255);
const FColor FColor::Magenta(255,0,255);
const FColor FColor::Orange(243, 156, 18);
const FColor FColor::Purple(169, 7, 228);
const FColor FColor::Turquoise(26, 188, 156);
const FColor FColor::Silver(189, 195, 199);
const FColor FColor::Emerald(46, 204, 113);

FLinearColor::FLinearColor(const FVector3f& Vector) :
	R(Vector.X),
	G(Vector.Y),
	B(Vector.Z),
	A(1.0f)
{}

FLinearColor::FLinearColor(const FVector3d& Vector) :
	R((float)Vector.X),
	G((float)Vector.Y),
	B((float)Vector.Z),
	A(1.0f)
{}

FLinearColor::FLinearColor(const FVector4f& Vector) :
	R(Vector.X),
	G(Vector.Y),
	B(Vector.Z),
	A(Vector.W)
{}

FLinearColor::FLinearColor(const FVector4d& Vector) :
	R((float)Vector.X),
	G((float)Vector.Y),
	B((float)Vector.Z),
	A((float)Vector.W)
{}

FLinearColor::FLinearColor(const FFloat16Color& C)
{
	*this = C.GetFloats();
}

FLinearColor FLinearColor::FromPow22Color(const FColor& Color)
{
	FLinearColor LinearColor;
	LinearColor.R = Pow22OneOver255Table[Color.R];
	LinearColor.G = Pow22OneOver255Table[Color.G];
	LinearColor.B =	Pow22OneOver255Table[Color.B];
	LinearColor.A =	float(Color.A) * (1.0f / 255.0f);

	return LinearColor;
}

/**
 * Converts from a linear float color to RGBE as outlined in Gregory Ward's Real Pixels article, Graphics Gems II, page 80.
 * Implementation details in https://cbloomrants.blogspot.com/2020/06/widespread-error-in-radiance-hdr-rgbe.html
 */ 
FColor FLinearColor::ToRGBE() const
{
	const float	Primary = FMath::Max3( R, G, B );

	if( Primary < 1E-32f )
	{
		return FColor(0,0,0,0);
	}
	else
	{
		// RGBE HDR can not store negative floats
		// Unreal passes negative floats here
		// just clamp them to zero :
		const float NonNegativeR = FMath::Max(R,0.f);
		const float NonNegativeG = FMath::Max(G,0.f);
		const float NonNegativeB = FMath::Max(B,0.f);

		// The following replaces a call to frexpf, because frexpf would have a warning for an unused return value.
		// Additionally, this usage of logbf assumes FLT_RADIX == 2
		int32 Exponent = 1 + (int32)logbf(Primary);
		const float Scale = ldexpf(1.f, -Exponent + 8);
		
		FColor	Color;
		// no clamp needed, should always fit in uint8 :
		Color.R = IntCastChecked<uint8>( (int)(NonNegativeR * Scale) );
		Color.G = IntCastChecked<uint8>( (int)(NonNegativeG * Scale) );
		Color.B = IntCastChecked<uint8>( (int)(NonNegativeB * Scale) );
		Color.A = IntCastChecked<uint8>( Exponent + 128 );
		return Color;
	}
}


// fast Linear to SRGB uint8 conversion
// https://gist.github.com/rygorous/2203834
//
// round-trips exactly
// quantization bucket boundaries vary by max of 0.11%
// biggest difference at i = 1
// thresholds[i] = 0.000456
// c_linear_float_srgb_thresholds[i] = 0.000455

typedef union
{
    uint32 u;
    float f;
} stbir__FP32;

static const uint32 stb_fp32_to_srgb8_tab4[104] = {
    0x0073000d, 0x007a000d, 0x0080000d, 0x0087000d, 0x008d000d, 0x0094000d, 0x009a000d, 0x00a1000d,
    0x00a7001a, 0x00b4001a, 0x00c1001a, 0x00ce001a, 0x00da001a, 0x00e7001a, 0x00f4001a, 0x0101001a,
    0x010e0033, 0x01280033, 0x01410033, 0x015b0033, 0x01750033, 0x018f0033, 0x01a80033, 0x01c20033,
    0x01dc0067, 0x020f0067, 0x02430067, 0x02760067, 0x02aa0067, 0x02dd0067, 0x03110067, 0x03440067,
    0x037800ce, 0x03df00ce, 0x044600ce, 0x04ad00ce, 0x051400ce, 0x057b00c5, 0x05dd00bc, 0x063b00b5,
    0x06970158, 0x07420142, 0x07e30130, 0x087b0120, 0x090b0112, 0x09940106, 0x0a1700fc, 0x0a9500f2,
    0x0b0f01cb, 0x0bf401ae, 0x0ccb0195, 0x0d950180, 0x0e56016e, 0x0f0d015e, 0x0fbc0150, 0x10630143,
    0x11070264, 0x1238023e, 0x1357021d, 0x14660201, 0x156601e9, 0x165a01d3, 0x174401c0, 0x182401af,
    0x18fe0331, 0x1a9602fe, 0x1c1502d2, 0x1d7e02ad, 0x1ed4028d, 0x201a0270, 0x21520256, 0x227d0240,
    0x239f0443, 0x25c003fe, 0x27bf03c4, 0x29a10392, 0x2b6a0367, 0x2d1d0341, 0x2ebe031f, 0x304d0300,
    0x31d105b0, 0x34a80555, 0x37520507, 0x39d504c5, 0x3c37048b, 0x3e7c0458, 0x40a8042a, 0x42bd0401,
    0x44c20798, 0x488e071e, 0x4c1c06b6, 0x4f76065d, 0x52a50610, 0x55ac05cc, 0x5892058f, 0x5b590559,
    0x5e0c0a23, 0x631c0980, 0x67db08f6, 0x6c55087f, 0x70940818, 0x74a007bd, 0x787d076c, 0x7c330723,
};
 
static uint8 stbir__linear_to_srgb_uchar_fast(float in)
{
    static const stbir__FP32 almostone = { 0x3f7fffff }; // 1-eps
    static const stbir__FP32 minval = { (127-13) << 23 };
    uint32 tab,bias,scale,t;
    stbir__FP32 f;
 
    // Clamp to [2^(-13), 1-eps]; these two values map to 0 and 1, respectively.
    // The tests are carefully written so that NaNs map to 0, same as in the reference
    // implementation.
    if (!(in > minval.f)) // written this way to catch NaNs
        in = minval.f;
    if (in > almostone.f)
        in = almostone.f;
 
    // Do the table lookup and unpack bias, scale
    f.f = in;
    tab = stb_fp32_to_srgb8_tab4[(f.u - minval.u) >> 20];
    bias = (tab >> 16) << 9;
    scale = tab & 0xffff;
 
    // Grab next-highest mantissa bits and perform linear interpolation
    t = (f.u >> 12) & 0xff;
    return (uint8) ((bias + scale*t) >> 16);
}

#if PLATFORM_CPU_X86_FAMILY && PLATFORM_ENABLE_VECTORINTRINSICS

static FColor ConvertLinearToSRGBSSE2(const FLinearColor& InColor)
{
	const VectorRegister4Float InRGBA = VectorLoad(&InColor.Component(0));

	// Clamp to [2^(-13), 1-eps]; these two values map to 0 and 1, respectively.
	// This clamping logic is carefully written so that NaNs map to 0.
	//
	// We do this clamping on all four color channels, even though we later handle
	// A differently; this does not change the results for A: 2^(-13) rounds to 0 in
	// U8, and 1-eps rounds to 255 in U8, so these are OK endpoints to use.
	const VectorRegister4Float AlmostOne = VectorCastIntToFloat(VectorIntSet1(0x3f7fffff)); // 1-eps
	const VectorRegister4Int MinValInt = VectorIntSet1((127 - 13) << 23);
	const VectorRegister4Float MinValFlt = VectorCastIntToFloat(MinValInt);

	const VectorRegister4Float InClamped = VectorMin(VectorMax(InRGBA, MinValFlt), AlmostOne);

	// Set up for the table lookup
	// This computes a 3-vector of table indices. The above clamping
	// ensures that the values in question are in [0,13*8-1]=[0,103].
	const VectorRegister4Int TabIndexVec = _mm_srli_epi32(_mm_sub_epi32(_mm_castps_si128(InClamped), MinValInt), 20);

	// Do the 4 table lookups with regular loads. We can use PEXTRW (SSE2)
	// to grab the 3 indices from lanes 1-3, lane 0 we can just get via MOVD.
	// The latter gives us a full 32 bits, not 16 like the other ones, but given
	// our value range either works.
	const VectorRegister4Int TabValR = _mm_cvtsi32_si128(stb_fp32_to_srgb8_tab4[(uint32)_mm_cvtsi128_si32(TabIndexVec)]);
	const VectorRegister4Int TabValG = _mm_cvtsi32_si128(stb_fp32_to_srgb8_tab4[(uint32)_mm_extract_epi16(TabIndexVec, 2)]);
	const VectorRegister4Int TabValB = _mm_cvtsi32_si128(stb_fp32_to_srgb8_tab4[(uint32)_mm_extract_epi16(TabIndexVec, 4)]);

	// Merge the four values we just loaded back into a 3-vector (gather complete!)
	const VectorRegister4Int TabValRG = _mm_unpacklo_epi32(TabValR, TabValG);
	const VectorRegister4Int TabValsRGB = _mm_unpacklo_epi64(TabValRG, TabValB); // This leaves A=0, which suits us

	// Grab the mantissa bits into the low 16 bits of each 32b lane, and set up 512 in the high
	// 16 bits of each 32b lane, which is how the bias values in the table are meant to be scaled.
	//
	// We grab mantissa bits [12,19] for the lerp.
	const VectorRegister4Int MantissaLerpFactor = _mm_and_si128(_mm_srli_epi32(_mm_castps_si128(InClamped), 12), _mm_set1_epi32(0xff));
	const VectorRegister4Int FinalMultiplier = _mm_or_si128(MantissaLerpFactor, _mm_set1_epi32(512 << 16));

	// In the table:
	//    (bias>>9) was stored in the high 16 bits
	//    scale was stored in the low 16 bits
	//    t = (mantissa >> 12) & 0xff
	//
	// then we want ((bias + scale*t) >> 16).
	// Except for the final shift, that's a single PMADDWD:
	const VectorRegister4Int InterpolatedRGB = _mm_srli_epi32(_mm_madd_epi16(TabValsRGB, FinalMultiplier), 16);

	// Finally, A gets done directly, via (int)(A * 255.f + 0.5f)
	// We zero out the non-A channels by multiplying by 0; our clamping earlier
	// took care of NaNs/infinites, so this is fine
	const VectorRegister4Float ScaledBiasedA = _mm_add_ps(_mm_mul_ps(InClamped, _mm_setr_ps(0.f, 0.f, 0.f, 255.f)), _mm_set1_ps(0.5f));
	const VectorRegister4Int FinalA  = _mm_cvttps_epi32(ScaledBiasedA);

	// Merge A into the result, reorder to BGRA, then pack down to bytes and store!
	// InterpolatedRGB has lane 3=0, and ComputedA has the first three lanes zero,
	// so we can just OR them together.
	const VectorRegister4Int FinalRGBA = _mm_or_si128(InterpolatedRGB, FinalA);
	const VectorRegister4Int FinalBGRA = _mm_shuffle_epi32(FinalRGBA, _MM_SHUFFLE(3, 0, 1, 2));

	const VectorRegister4Int Packed16 = _mm_packs_epi32(FinalBGRA, FinalBGRA);
	const VectorRegister4Int Packed8 = _mm_packus_epi16(Packed16, Packed16);

	return FColor((uint32)_mm_cvtsi128_si32(Packed8));
}

#endif

/** Quantizes the linear color and returns the result as a FColor with optional sRGB conversion and quality as goal. */
FColor FLinearColor::ToFColorSRGB() const
{
	// The convention used here in all channels is that NaNs
	// convert to 0, as do negative values, and out-of-range
	// positive values convert to 255.

#if PLATFORM_CPU_X86_FAMILY && PLATFORM_ENABLE_VECTORINTRINSICS
	return ConvertLinearToSRGBSSE2(*this);
#else
	return FColor(
		stbir__linear_to_srgb_uchar_fast(R),
		stbir__linear_to_srgb_uchar_fast(G),
		stbir__linear_to_srgb_uchar_fast(B),
		(uint8)(0.5f + Clamp01NansTo0(A)*255.f)
	);
#endif
}

/**
 * Convert multiple FLinearColors to sRGB FColor; array version of FLinearColor::ToFColorSRGB.
 * 
 * @param	InLinearColors	Pointer to one or more FLinearColors to convert.
 * @param	OutColorsSRGB	Pointer to one or more FColors that receive the results.
 * @param	InCount			Number of colors to convert.
 */
void ConvertFLinearColorsToFColorSRGB(const FLinearColor* InLinearColors, FColor* OutColorsSRGB, int64 InCount)
{
	// This function exists because calling FLinearColor::ToFColorSRGB()
	// from another module is not cheap. However, we're in the same module
	// as the definition here, so inlining works fine, and we can just
	// implement this as the straightforward loop.
	for (int64 Index = 0; Index < InCount; ++Index)
	{
		OutColorsSRGB[Index] = InLinearColors[Index].ToFColorSRGB();
	}
}

/**
 * Returns a desaturated color, with 0 meaning no desaturation and 1 == full desaturation
 *
 * @param	Desaturation	Desaturation factor in range [0..1]
 * @return	Desaturated color
 */
FLinearColor FLinearColor::Desaturate( float Desaturation ) const
{
	float Lum = GetLuminance();
	return FMath::Lerp( *this, FLinearColor( Lum, Lum, Lum, 0 ), Desaturation );
}

/** Computes the perceptually weighted luminance value of a color. */
float FLinearColor::GetLuminance() const
{
	static const IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.LegacyLuminanceFactors"));
	static const FLinearColor LuminanceFactors = CVar->GetInt() != 0 ? FLinearColor(0.3f, 0.59f, 0.11f) : UE::Color::FColorSpace::GetWorking().GetLuminanceFactors();

	return R * LuminanceFactors.R + G * LuminanceFactors.G + B * LuminanceFactors.B;
}

FColor FColor::FromHex( const FString& HexString )
{
	int32 StartIndex = (!HexString.IsEmpty() && HexString[0] == TCHAR('#')) ? 1 : 0;

	if (HexString.Len() == 3 + StartIndex)
	{
		const int32 R = FParse::HexDigit(HexString[StartIndex++]);
		const int32 G = FParse::HexDigit(HexString[StartIndex++]);
		const int32 B = FParse::HexDigit(HexString[StartIndex]);

		return FColor((uint8)((R << 4) + R), (uint8)((G << 4) + G), (uint8)((B << 4) + B), 255);
	}

	if (HexString.Len() == 6 + StartIndex)
	{
		FColor Result;

		Result.R = (uint8)((FParse::HexDigit(HexString[StartIndex+0]) << 4) + FParse::HexDigit(HexString[StartIndex+1]));
		Result.G = (uint8)((FParse::HexDigit(HexString[StartIndex+2]) << 4) + FParse::HexDigit(HexString[StartIndex+3]));
		Result.B = (uint8)((FParse::HexDigit(HexString[StartIndex+4]) << 4) + FParse::HexDigit(HexString[StartIndex+5]));
		Result.A = 255;

		return Result;
	}

	if (HexString.Len() == 8 + StartIndex)
	{
		FColor Result;

		Result.R = (uint8)((FParse::HexDigit(HexString[StartIndex+0]) << 4) + FParse::HexDigit(HexString[StartIndex+1]));
		Result.G = (uint8)((FParse::HexDigit(HexString[StartIndex+2]) << 4) + FParse::HexDigit(HexString[StartIndex+3]));
		Result.B = (uint8)((FParse::HexDigit(HexString[StartIndex+4]) << 4) + FParse::HexDigit(HexString[StartIndex+5]));
		Result.A = (uint8)((FParse::HexDigit(HexString[StartIndex+6]) << 4) + FParse::HexDigit(HexString[StartIndex+7]));

		return Result;
	}

	return FColor(ForceInitToZero);
}

/**
 * Converts from RGBE to a linear float color as outlined in Gregory Ward's Real Pixels article, Graphics Gems II, page 80.
 * Implementation details in https://cbloomrants.blogspot.com/2020/06/widespread-error-in-radiance-hdr-rgbe.html
 */ 
FLinearColor FColor::FromRGBE() const
{
	if (A == 0)
	{
		return FLinearColor::Black;
	}
	else
	{
		// the extra 8 here does the /256
		const float Scale = ldexpf(1.f, A - (128 + 8));
		// bias by 0.5 so [0,255] input shifts to [0.5,255.5] making it centered after /256
		return FLinearColor( 
			(R + 0.5f) * Scale, 
			(G + 0.5f) * Scale, 
			(B + 0.5f) * Scale, 
			1.0f );
	}
}

/**
 * Converts byte hue-saturation-brightness to floating point red-green-blue.
 */
FLinearColor FLinearColor::MakeFromHSV8(uint8 H, uint8 S, uint8 V)
{
	// want a given H value of 255 to map to just below 360 degrees
	const FLinearColor HSVColor((float)H * (360.0f / 256.0f), (float)S / 255.0f, (float)V / 255.0f);
	return HSVColor.HSVToLinearRGB();
}

/** Converts a linear space RGB color to an HSV color */
FLinearColor FLinearColor::LinearRGBToHSV() const
{
	const float RGBMin = FMath::Min3(R, G, B);
	const float RGBMax = FMath::Max3(R, G, B);
	const float RGBRange = RGBMax - RGBMin;

	if ( RGBMax <= 0.f )
	{
		return FLinearColor(0.f,0.f,0.f,A);
	}
	// RGBMin can be negative ; while that doesn't work great, it also doesn't break the math below

	const float Hue = (RGBMax == RGBMin ? 0.0f :
					   RGBMax == R    ? FMath::Fmod((((G - B) / RGBRange) * 60.0f) + 360.0f, 360.0f) :
					   RGBMax == G    ?             (((B - R) / RGBRange) * 60.0f) + 120.0f :
					   RGBMax == B    ?             (((R - G) / RGBRange) * 60.0f) + 240.0f :
					   0.0f);
	
	const float Saturation = (RGBMax == 0.0f ? 0.0f : RGBRange / RGBMax);
	const float Value = RGBMax;

	// In the new color, R = H, G = S, B = V, A = A
	return FLinearColor(Hue, Saturation, Value, A);
}



/** Converts an HSV color to a linear space RGB color */
FLinearColor FLinearColor::HSVToLinearRGB() const
{
	// In this color, R = H, G = S, B = V  ; H in [0,360] , S&V in [0,1]
	float Hue = fmodf(R,360.f);
	if ( Hue < 0.f ) Hue += 360.f;
	const float Saturation = FMath::Clamp(G,0.f,1.f);
	const float Value = B; // Value can be out of [0,1]

	const float HDiv60 = Hue / 60.0f;
	const uint32 HDiv60_Floor = (uint32)(HDiv60);
	const float HDiv60_Fraction = HDiv60 - (float)HDiv60_Floor;

	const float RGBValues[4] = {
		Value,
		Value * (1.0f - Saturation),
		Value * (1.0f - (HDiv60_Fraction * Saturation)),
		Value * (1.0f - ((1.0f - HDiv60_Fraction) * Saturation)),
	};
	const uint32 RGBSwizzle[6][3] = {
		{0, 3, 1},
		{2, 0, 1},
		{1, 0, 3},
		{1, 2, 0},
		{3, 1, 0},
		{0, 1, 2},
	};
	const uint32 SwizzleIndex = HDiv60_Floor % 6;

	return FLinearColor(RGBValues[RGBSwizzle[SwizzleIndex][0]],
						RGBValues[RGBSwizzle[SwizzleIndex][1]],
						RGBValues[RGBSwizzle[SwizzleIndex][2]],
						A);
}

FLinearColor FLinearColor::LerpUsingHSV( const FLinearColor& From, const FLinearColor& To, const float Progress )
{
	const FLinearColor FromHSV = From.LinearRGBToHSV();
	const FLinearColor ToHSV = To.LinearRGBToHSV();

	float FromHue = FromHSV.R;
	float ToHue = ToHSV.R;

	// Take the shortest path to the new hue
	if( FMath::Abs( FromHue - ToHue ) > 180.0f )
	{
		if( ToHue > FromHue )
		{
			FromHue += 360.0f;
		}
		else
		{
			ToHue += 360.0f;
		}
	}

	float NewHue = FMath::Lerp( FromHue, ToHue, Progress );

	NewHue = FMath::Fmod( NewHue, 360.0f );
	if( NewHue < 0.0f )
	{
		NewHue += 360.0f;
	}

	const float NewSaturation = FMath::Lerp( FromHSV.G, ToHSV.G, Progress );
	const float NewValue = FMath::Lerp( FromHSV.B, ToHSV.B, Progress );
	FLinearColor Interpolated = FLinearColor( NewHue, NewSaturation, NewValue ).HSVToLinearRGB();

	const float NewAlpha = FMath::Lerp( From.A, To.A, Progress );
	Interpolated.A = NewAlpha;

	return Interpolated;
}


/**
* Makes a random but quite nice color.
*/
FLinearColor FLinearColor::MakeRandomColor()
{
	// step hue around the ring, with a step relatively prime to 256
	// this ensures that many calls in a row of this function produce very different values
	static std::atomic<uint32> s_hue(0);
	uint32 Hue = s_hue.fetch_add(157,std::memory_order_relaxed);
	// 157 and 181 are both good steps

	return FLinearColor::MakeFromHSV8((uint8)Hue, 255, 255);
}

FColor FColor::MakeRandomColor()
{
	return FLinearColor::MakeRandomColor().ToFColor(true);
}

FLinearColor FLinearColor::MakeFromColorTemperature( float Temp )
{
	Temp = FMath::Clamp( Temp, 1000.0f, 15000.0f );

	// Approximate Planckian locus in CIE 1960 UCS
	float u = ( 0.860117757f + 1.54118254e-4f * Temp + 1.28641212e-7f * Temp*Temp ) / ( 1.0f + 8.42420235e-4f * Temp + 7.08145163e-7f * Temp*Temp );
	float v = ( 0.317398726f + 4.22806245e-5f * Temp + 4.20481691e-8f * Temp*Temp ) / ( 1.0f - 2.89741816e-5f * Temp + 1.61456053e-7f * Temp*Temp );

	float x = 3.0f * u / ( 2.0f * u - 8.0f * v + 4.0f );
	float y = 2.0f * v / ( 2.0f * u - 8.0f * v + 4.0f );
	float z = 1.0f - x - y;

	float Y = 1.0f;
	float X = Y/y * x;
	float Z = Y/y * z;

	// XYZ to RGB with BT.709 primaries
	float R =  3.2404542f * X + -1.5371385f * Y + -0.4985314f * Z;
	float G = -0.9692660f * X +  1.8760108f * Y +  0.0415560f * Z;
	float B =  0.0556434f * X + -0.2040259f * Y +  1.0572252f * Z;

	// The XYZ to RGB transform can result in negative values, so we need to clamp here.
	return FLinearColor(FMath::Max(0.0f, R), FMath::Max(0.0f, G), FMath::Max(0.0f, B));
}

FLinearColor FLinearColor::MakeRandomSeededColor(int32 Seed)
{
	FRandomStream RandomStream(Seed);

	float R = RandomStream.GetFraction();
	float G = RandomStream.GetFraction();
	float B = RandomStream.GetFraction();

	return FLinearColor(R,G,B);
}

FString FLinearColor::ToString() const
{
	return FString::Printf(TEXT("(R=%f,G=%f,B=%f,A=%f)"), R, G, B, A);
}

bool FLinearColor::InitFromString(const FString& InSourceString)
{
	R = G = B = 0.f;
	A = 1.f;

	// The initialization is only successful if the R, G, and B values can all be parsed from the string
	const bool bSuccessful = FParse::Value(*InSourceString, TEXT("R="), R) && FParse::Value(*InSourceString, TEXT("G="), G) && FParse::Value(*InSourceString, TEXT("B="), B);

	// Alpha is optional, so don't factor in its presence (or lack thereof) in determining initialization success
	FParse::Value(*InSourceString, TEXT("A="), A);

	return bSuccessful;
}

FColor FColor::MakeRandomSeededColor(int32 Seed)
{
	return FLinearColor::MakeRandomSeededColor( Seed ).ToFColor( true );
}

FColor FColor::MakeFromColorTemperature( float Temp )
{
	return FLinearColor::MakeFromColorTemperature( Temp ).ToFColor( true );
}

FColor FColor::MakeRedToGreenColorFromScalar(float Scalar)
{
	const float RedSclr = FMath::Clamp<float>((1.0f - Scalar)/0.5f,0.f,1.f);
	const float GreenSclr = FMath::Clamp<float>((Scalar/0.5f),0.f,1.f);
	const uint8 R = (uint8)FMath::TruncToInt(255 * RedSclr);
	const uint8 G = (uint8)FMath::TruncToInt(255 * GreenSclr);
	const uint8 B = 0;
	return FColor(R, G, B);
}

FString FColor::ToHex() const
{
	return FString::Printf(TEXT("%02X%02X%02X%02X"), R, G, B, A);
}

FString FColor::ToString() const
{
	return FString::Printf(TEXT("(R=%i,G=%i,B=%i,A=%i)"), R, G, B, A);
}

bool FColor::InitFromString(const FString& InSourceString)
{
	R = G = B = 0;
	A = 255;

	// The initialization is only successful if the R, G, and B values can all be parsed from the string
	const bool bSuccessful = FParse::Value(*InSourceString, TEXT("R="), R) && FParse::Value(*InSourceString, TEXT("G="), G) && FParse::Value(*InSourceString, TEXT("B="), B);

	// Alpha is optional, so don't factor in its presence (or lack thereof) in determining initialization success
	FParse::Value(*InSourceString, TEXT("A="), A);

	return bSuccessful;
}

void ComputeAndFixedColorAndIntensity(const FLinearColor& InLinearColor,FColor& OutColor,float& OutIntensity)
{
	float MaxComponent = FMath::Max(UE_DELTA,FMath::Max(InLinearColor.R,FMath::Max(InLinearColor.G,InLinearColor.B)));
	OutColor = ( InLinearColor / MaxComponent ).ToFColor(true);
	OutIntensity = MaxComponent;
}


/**
 * Pow table for fast FColor -> FLinearColor conversion.
 *
 * FMath::Pow( i / 255.f, 2.2f )
 */
float FLinearColor::Pow22OneOver255Table[256] =
{
	0.0f, 5.07705190066176E-06f, 2.33280046660989E-05f, 5.69217657121931E-05f, 0.000107187362341244f, 0.000175123977503027f, 0.000261543754548491f, 0.000367136269815943f, 0.000492503787191433f,
	0.000638182842167022f, 0.000804658499513058f, 0.000992374304074325f, 0.0012017395224384f, 0.00143313458967186f, 0.00168691531678928f, 0.00196341621339647f, 0.00226295316070643f,
	0.00258582559623417f, 0.00293231832393836f, 0.00330270303200364f, 0.00369723957890013f, 0.00411617709328275f, 0.00455975492252602f, 0.00502820345685554f, 0.00552174485023966f,
	0.00604059365484981f, 0.00658495738258168f, 0.00715503700457303f, 0.00775102739766061f, 0.00837311774514858f, 0.00902149189801213f, 0.00969632870165823f, 0.0103978022925553f,
	0.0111260823683832f, 0.0118813344348137f, 0.0126637200315821f, 0.0134733969401426f, 0.0143105193748841f, 0.0151752381596252f, 0.0160677008908869f, 0.01698805208925f, 0.0179364333399502f,
	0.0189129834237215f, 0.0199178384387857f, 0.0209511319147811f, 0.0220129949193365f, 0.0231035561579214f, 0.0242229420675342f, 0.0253712769047346f, 0.0265486828284729f, 0.027755279978126f,
	0.0289911865471078f, 0.0302565188523887f, 0.0315513914002264f, 0.0328759169483838f, 0.034230206565082f, 0.0356143696849188f, 0.0370285141619602f, 0.0384727463201946f, 0.0399471710015256f,
	0.0414518916114625f, 0.0429870101626571f, 0.0445526273164214f, 0.0461488424223509f, 0.0477757535561706f, 0.049433457555908f, 0.0511220500564934f, 0.052841625522879f, 0.0545922772817603f,
	0.0563740975519798f, 0.0581871774736854f, 0.0600316071363132f, 0.0619074756054558f, 0.0638148709486772f, 0.0657538802603301f, 0.0677245896854243f, 0.0697270844425988f, 0.0717614488462391f,
	0.0738277663277846f, 0.0759261194562648f, 0.0780565899581019f, 0.080219258736215f, 0.0824142058884592f, 0.0846415107254295f, 0.0869012517876603f, 0.0891935068622478f, 0.0915183529989195f,
	0.0938758665255778f, 0.0962661230633397f, 0.0986891975410945f, 0.1011451642096f, 0.103634096655137f, 0.106156067812744f, 0.108711149979039f, 0.11129941482466f, 0.113920933406333f,
	0.116575776178572f, 0.119264013005047f, 0.121985713169619f, 0.124740945387051f, 0.127529777813422f, 0.130352278056244f, 0.1332085131843f, 0.136098549737202f, 0.139022453734703f,
	0.141980290685736f, 0.144972125597231f, 0.147998022982685f, 0.151058046870511f, 0.154152260812165f, 0.157280727890073f, 0.160443510725344f, 0.16364067148529f, 0.166872271890766f,
	0.170138373223312f, 0.173439036332135f, 0.176774321640903f, 0.18014428915439f, 0.183548998464951f, 0.186988508758844f, 0.190462878822409f, 0.193972167048093f, 0.19751643144034f,
	0.201095729621346f, 0.204710118836677f, 0.208359655960767f, 0.212044397502288f, 0.215764399609395f, 0.219519718074868f, 0.223310408341127f, 0.227136525505149f, 0.230998124323267f,
	0.23489525921588f, 0.238827984272048f, 0.242796353254002f, 0.24680041960155f, 0.2508402364364f, 0.254915856566385f, 0.259027332489606f, 0.263174716398492f, 0.267358060183772f,
	0.271577415438375f, 0.275832833461245f, 0.280124365261085f, 0.284452061560024f, 0.288815972797219f, 0.293216149132375f, 0.297652640449211f, 0.302125496358853f, 0.306634766203158f,
	0.311180499057984f, 0.315762743736397f, 0.32038154879181f, 0.325036962521076f, 0.329729032967515f, 0.334457807923889f, 0.339223334935327f, 0.344025661302187f, 0.348864834082879f,
	0.353740900096629f, 0.358653905926199f, 0.363603897920553f, 0.368590922197487f, 0.373615024646202f, 0.37867625092984f, 0.383774646487975f, 0.388910256539059f, 0.394083126082829f,
	0.399293299902674f, 0.404540822567962f, 0.409825738436323f, 0.415148091655907f, 0.420507926167587f, 0.425905285707146f, 0.43134021380741f, 0.436812753800359f, 0.442322948819202f,
	0.44787084180041f, 0.453456475485731f, 0.45907989242416f, 0.46474113497389f, 0.470440245304218f, 0.47617726539744f, 0.481952237050698f, 0.487765201877811f, 0.493616201311074f,
	0.49950527660303f, 0.505432468828216f, 0.511397818884879f, 0.517401367496673f, 0.523443155214325f, 0.529523222417277f, 0.535641609315311f, 0.541798355950137f, 0.547993502196972f,
	0.554227087766085f, 0.560499152204328f, 0.566809734896638f, 0.573158875067523f, 0.579546611782525f, 0.585972983949661f, 0.592438030320847f, 0.598941789493296f, 0.605484299910907f,
	0.612065599865624f, 0.61868572749878f, 0.625344720802427f, 0.632042617620641f, 0.638779455650817f, 0.645555272444934f, 0.652370105410821f, 0.659223991813387f, 0.666116968775851f,
	0.673049073280942f, 0.680020342172095f, 0.687030812154625f, 0.694080519796882f, 0.701169501531402f, 0.708297793656032f, 0.715465432335048f, 0.722672453600255f, 0.729918893352071f,
	0.737204787360605f, 0.744530171266715f, 0.751895080583051f, 0.759299550695091f, 0.766743616862161f, 0.774227314218442f, 0.781750677773962f, 0.789313742415586f, 0.796916542907978f,
	0.804559113894567f, 0.81224148989849f, 0.819963705323528f, 0.827725794455034f, 0.835527791460841f, 0.843369730392169f, 0.851251645184515f, 0.859173569658532f, 0.867135537520905f,
	0.875137582365205f, 0.883179737672745f, 0.891262036813419f, 0.899384513046529f, 0.907547199521614f, 0.915750129279253f, 0.923993335251873f, 0.932276850264543f, 0.940600707035753f,
	0.948964938178195f, 0.957369576199527f, 0.96581465350313f, 0.974300202388861f, 0.982826255053791f, 0.99139284359294f, 1.0f
};
