// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ColorList.cpp: List of common colors implementation.
=============================================================================*/

#include "Math/ColorList.h"

#include "Containers/UnrealString.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Trace/Detail/Channel.h"

DEFINE_LOG_CATEGORY_STATIC(LogColorList, Log, All);

/** Global instance of color list helper class. */
FColorList GColorList;

const FColor& FColorList::GetFColorByName( const TCHAR* ColorName ) const
{
	const FColor* Color = ColorsMap.FindRef( ColorName );

	if( Color != NULL )
	{
		return *Color;
	}
	
	return White;
}

const FColor& FColorList::GetFColorByIndex( int32 ColorIndex ) const
{
	if( ColorsLookup.IsValidIndex( ColorIndex ) == true )
	{
		return *ColorsLookup[ ColorIndex ];
	}
	
	return White;
}

const FLinearColor FColorList::GetFLinearColorByName( const TCHAR* ColorName ) const
{
	const FColor* Color = ColorsMap.FindRef( ColorName );

	if( Color != NULL )
	{
		return FLinearColor( *Color );
	}

	return FLinearColor::White;
}

bool FColorList::IsValidColorName( const TCHAR* ColorName ) const
{
	const FColor* Color = ColorsMap.FindRef( ColorName );
	return Color != NULL ? true : false;
}


int32 FColorList::GetColorIndex( const TCHAR* ColorName ) const
{
	const FColor& Color = GetFColorByName( ColorName );
	int32 ColorIndex = 0;
	ColorsLookup.Find( &Color, ColorIndex );
	return ColorIndex;
}

const FString& FColorList::GetColorNameByIndex( int32 ColorIndex ) const
{
	static const FString BadIndex( TEXT( "BadIndex" ) );

	if( ColorsLookup.IsValidIndex( ColorIndex ) == true )
	{
		const FColor& Color = *ColorsLookup[ ColorIndex ];
		const FString& ColorName = *ColorsMap.FindKey( &Color );
		return ColorName;
	}

	return BadIndex;
}

void FColorList::CreateColorMap()
{
	int32 Index = 0;
	// Black color must be first.
	InitializeColor( TEXT("black"),				&FColorList::Black, Index );
	InitializeColor( TEXT("aquamarine"),		&FColorList::Aquamarine, Index );
	InitializeColor( TEXT("bakerchocolate"),	&FColorList::BakerChocolate, Index );
	InitializeColor( TEXT("blue"),				&FColorList::Blue, Index );
	InitializeColor( TEXT("blueviolet"),		&FColorList::BlueViolet, Index );
	InitializeColor( TEXT("brass"),				&FColorList::Brass, Index );
	InitializeColor( TEXT("brightgold"),		&FColorList::BrightGold, Index );
	InitializeColor( TEXT("brown"),				&FColorList::Brown, Index );
	InitializeColor( TEXT("bronze"),			&FColorList::Bronze, Index );
	InitializeColor( TEXT("bronzeii"),			&FColorList::BronzeII, Index );
	InitializeColor( TEXT("cadetblue"),			&FColorList::CadetBlue, Index );
	InitializeColor( TEXT("coolcopper"),		&FColorList::CoolCopper, Index );
	InitializeColor( TEXT("copper"),			&FColorList::Copper, Index );
	InitializeColor( TEXT("coral"),				&FColorList::Coral, Index );
	InitializeColor( TEXT("cornflowerblue"),	&FColorList::CornFlowerBlue, Index );
	InitializeColor( TEXT("cyan"),				&FColorList::Cyan, Index );
	InitializeColor( TEXT("darkbrown"),			&FColorList::DarkBrown, Index );
	InitializeColor( TEXT("darkgreen"),			&FColorList::DarkGreen, Index );
	InitializeColor( TEXT("darkgreencopper"),	&FColorList::DarkGreenCopper, Index );
	InitializeColor( TEXT("darkolivegreen"),	&FColorList::DarkOliveGreen, Index );
	InitializeColor( TEXT("darkorchid"),		&FColorList::DarkOrchid, Index );
	InitializeColor( TEXT("darkpurple"),		&FColorList::DarkPurple, Index );
	InitializeColor( TEXT("darkslateblue"),		&FColorList::DarkSlateBlue, Index );
	InitializeColor( TEXT("darkslategrey"),		&FColorList::DarkSlateGrey, Index );
	InitializeColor( TEXT("darktan"),			&FColorList::DarkTan, Index );
	InitializeColor( TEXT("darkturquoise"),		&FColorList::DarkTurquoise, Index );
	InitializeColor( TEXT("darkwood"),			&FColorList::DarkWood, Index );
	InitializeColor( TEXT("dimgrey"),			&FColorList::DimGrey, Index );
	InitializeColor( TEXT("dustyrose"),			&FColorList::DustyRose, Index );
	InitializeColor( TEXT("feldspar"),			&FColorList::Feldspar, Index );
	InitializeColor( TEXT("firebrick"),			&FColorList::Firebrick, Index );
	InitializeColor( TEXT("forestgreen"),		&FColorList::ForestGreen, Index );
	InitializeColor( TEXT("gold"),				&FColorList::Gold, Index );
	InitializeColor( TEXT("goldenrod"),			&FColorList::Goldenrod, Index );
	InitializeColor( TEXT("green"),				&FColorList::Green, Index );
	InitializeColor( TEXT("greencopper"),		&FColorList::GreenCopper, Index );
	InitializeColor( TEXT("greenyellow"),		&FColorList::GreenYellow, Index );
	InitializeColor( TEXT("grey"),				&FColorList::Grey, Index );
	InitializeColor( TEXT("huntergreen"),		&FColorList::HunterGreen, Index );
	InitializeColor( TEXT("indianred"),			&FColorList::IndianRed, Index );
	InitializeColor( TEXT("khaki"),				&FColorList::Khaki, Index );
	InitializeColor( TEXT("lightblue"),			&FColorList::LightBlue, Index );
	InitializeColor( TEXT("lightgrey"),			&FColorList::LightGrey, Index );
	InitializeColor( TEXT("lightsteelblue"),	&FColorList::LightSteelBlue, Index );
	InitializeColor( TEXT("lightwood"),			&FColorList::LightWood, Index );
	InitializeColor( TEXT("limegreen"),			&FColorList::LimeGreen, Index );
	InitializeColor( TEXT("magenta"),			&FColorList::Magenta, Index );
	InitializeColor( TEXT("mandarianorange"),	&FColorList::MandarianOrange, Index );
	InitializeColor( TEXT("maroon"),			&FColorList::Maroon, Index );
	InitializeColor( TEXT("mediumaquamarine"),	&FColorList::MediumAquamarine, Index );
	InitializeColor( TEXT("mediumblue"),		&FColorList::MediumBlue, Index );
	InitializeColor( TEXT("mediumforestgreen"), &FColorList::MediumForestGreen, Index );
	InitializeColor( TEXT("mediumgoldenrod"),	&FColorList::MediumGoldenrod, Index );
	InitializeColor( TEXT("mediumorchid"),		&FColorList::MediumOrchid, Index );
	InitializeColor( TEXT("mediumseagreen"),	&FColorList::MediumSeaGreen, Index );
	InitializeColor( TEXT("mediumslateblue"),	&FColorList::MediumSlateBlue, Index );
	InitializeColor( TEXT("mediumspringgreen"), &FColorList::MediumSpringGreen, Index );
	InitializeColor( TEXT("mediumturquoise"),	&FColorList::MediumTurquoise, Index );
	InitializeColor( TEXT("mediumvioletred"),	&FColorList::MediumVioletRed, Index );
	InitializeColor( TEXT("mediumwood"),		&FColorList::MediumWood, Index );
	InitializeColor( TEXT("midnightblue"),		&FColorList::MidnightBlue, Index );
	InitializeColor( TEXT("navyblue"),			&FColorList::NavyBlue, Index );
	InitializeColor( TEXT("neonblue"),			&FColorList::NeonBlue, Index );
	InitializeColor( TEXT("neonpink"),			&FColorList::NeonPink, Index );
	InitializeColor( TEXT("newmidnightblue"),	&FColorList::NewMidnightBlue, Index );
	InitializeColor( TEXT("newtan"),			&FColorList::NewTan, Index );
	InitializeColor( TEXT("oldgold"),			&FColorList::OldGold, Index );
	InitializeColor( TEXT("orange"),			&FColorList::Orange, Index );
	InitializeColor( TEXT("orangered"),			&FColorList::OrangeRed, Index );
	InitializeColor( TEXT("orchid"),			&FColorList::Orchid, Index );
	InitializeColor( TEXT("palegreen"),			&FColorList::PaleGreen, Index );
	InitializeColor( TEXT("pink"),				&FColorList::Pink, Index );
	InitializeColor( TEXT("plum"),				&FColorList::Plum, Index );
	InitializeColor( TEXT("quartz"),			&FColorList::Quartz, Index );
	InitializeColor( TEXT("red"),				&FColorList::Red, Index );
	InitializeColor( TEXT("richblue"),			&FColorList::RichBlue, Index );
	InitializeColor( TEXT("salmon"),			&FColorList::Salmon, Index );
	InitializeColor( TEXT("scarlet"),			&FColorList::Scarlet, Index );
	InitializeColor( TEXT("seagreen"),			&FColorList::SeaGreen, Index );
	InitializeColor( TEXT("semisweetchocolate"), &FColorList::SemiSweetChocolate, Index );
	InitializeColor( TEXT("sienna"),			&FColorList::Sienna, Index );
	InitializeColor( TEXT("silver"),			&FColorList::Silver, Index );
	InitializeColor( TEXT("skyblue"),			&FColorList::SkyBlue, Index );
	InitializeColor( TEXT("slateblue"),			&FColorList::SlateBlue, Index );
	InitializeColor( TEXT("spicypink"),			&FColorList::SpicyPink, Index );
	InitializeColor( TEXT("springgreen"),		&FColorList::SpringGreen, Index );
	InitializeColor( TEXT("steelblue"),			&FColorList::SteelBlue, Index );
	InitializeColor( TEXT("summersky"),			&FColorList::SummerSky, Index );
	InitializeColor( TEXT("tan"),				&FColorList::Tan, Index );
	InitializeColor( TEXT("thistle"),			&FColorList::Thistle, Index );
	InitializeColor( TEXT("turquoise"),			&FColorList::Turquoise, Index );
	InitializeColor( TEXT("verydarkbrown"),		&FColorList::VeryDarkBrown, Index );
	InitializeColor( TEXT("verylightgrey"),		&FColorList::VeryLightGrey, Index );
	InitializeColor( TEXT("violet"),			&FColorList::Violet, Index );
	InitializeColor( TEXT("violetred"),			&FColorList::VioletRed, Index );
	InitializeColor( TEXT("wheat"),				&FColorList::Wheat, Index );
	InitializeColor( TEXT("white"),				&FColorList::White, Index );
	InitializeColor( TEXT("yellow"),			&FColorList::Yellow, Index );
	InitializeColor( TEXT("yellowgreen"),		&FColorList::YellowGreen, Index );

	ColorsLookup.Shrink();
}

void FColorList::InitializeColor( const TCHAR* ColorName, const FColor* ColorPtr, int32& CurrentIndex )
{
	ColorsMap.Add( ColorName, ColorPtr );
	ColorsLookup.Add( ColorPtr );

	CurrentIndex ++;
}

void FColorList::LogColors()
{
	for( TColorsMap::TIterator It(ColorsMap); It; ++It )
	{
		const FColor* ColorPtr = It.Value();
		const FString& ColorName = It.Key();

		int32 ColorIndex = 0;
		ColorsLookup.Find( ColorPtr, ColorIndex );

		UE_LOGF(LogColorList, Log,  "%3i - %32ls -> %ls", ColorIndex, *ColorName, *ColorPtr->ToString() );
	}
}

// Common colors declarations.
const FColor FColorList::White            ( 255, 255, 255, 255 );
const FColor FColorList::Red              ( 255,   0,   0, 255 );
const FColor FColorList::Green            (   0, 255,   0, 255 );
const FColor FColorList::Blue             (   0,   0, 255, 255 );
const FColor FColorList::Magenta          ( 255,   0, 255, 255 );
const FColor FColorList::Cyan             (   0, 255, 255, 255 );
const FColor FColorList::Yellow           ( 255, 255,   0, 255 );
const FColor FColorList::Black            (   0,   0,   0, 255 );
const FColor FColorList::Aquamarine       ( 112, 219, 147, 255 );
const FColor FColorList::BakerChocolate   (  92,  51,  23, 255 );
const FColor FColorList::BlueViolet       ( 159,  95, 159, 255 );
const FColor FColorList::Brass            ( 181, 166,  66, 255 );
const FColor FColorList::BrightGold       ( 217, 217,  25, 255 );
const FColor FColorList::Brown            ( 166,  42,  42, 255 );
const FColor FColorList::Bronze           ( 140, 120,  83, 255 );
const FColor FColorList::BronzeII         ( 166, 125,  61, 255 );
const FColor FColorList::CadetBlue        (  95, 159, 159, 255 );
const FColor FColorList::CoolCopper       ( 217, 135,  25, 255 );
const FColor FColorList::Copper           ( 184, 115,  51, 255 );
const FColor FColorList::Coral            ( 255, 127,   0, 255 );
const FColor FColorList::CornFlowerBlue   (  66,  66, 111, 255 );
const FColor FColorList::DarkBrown        (  92,  64,  51, 255 );
const FColor FColorList::DarkGreen        (  47,  79,  47, 255 );
const FColor FColorList::DarkGreenCopper  (  74, 118, 110, 255 );
const FColor FColorList::DarkOliveGreen   (  79,  79,  47, 255 );
const FColor FColorList::DarkOrchid       ( 153,  50, 205, 255 );
const FColor FColorList::DarkPurple       ( 135,  31, 120, 255 );
const FColor FColorList::DarkSlateBlue    ( 107,  35, 142, 255 );
const FColor FColorList::DarkSlateGrey    (  47,  79,  79, 255 );
const FColor FColorList::DarkTan          ( 151, 105,  79, 255 );
const FColor FColorList::DarkTurquoise    ( 112, 147, 219, 255 );
const FColor FColorList::DarkWood         ( 133,  94,  66, 255 );
const FColor FColorList::DimGrey          (  84,  84,  84, 255 );
const FColor FColorList::DustyRose        ( 133,  99,  99, 255 );
const FColor FColorList::Feldspar         ( 209, 146, 117, 255 );
const FColor FColorList::Firebrick        ( 142,  35,  35, 255 );
const FColor FColorList::ForestGreen      (  35, 142,  35, 255 );
const FColor FColorList::Gold             ( 205, 127,  50, 255 );
const FColor FColorList::Goldenrod        ( 219, 219, 112, 255 );
const FColor FColorList::Grey             ( 192, 192, 192, 255 );
const FColor FColorList::GreenCopper      (  82, 127, 118, 255 );
const FColor FColorList::GreenYellow      ( 147, 219, 112, 255 );
const FColor FColorList::HunterGreen      (  33,  94,  33, 255 );
const FColor FColorList::IndianRed        (  78,  47,  47, 255 );
const FColor FColorList::Khaki            ( 159, 159,  95, 255 );
const FColor FColorList::LightBlue        ( 192, 217, 217, 255 );
const FColor FColorList::LightGrey        ( 168, 168, 168, 255 );
const FColor FColorList::LightSteelBlue   ( 143, 143, 189, 255 );
const FColor FColorList::LightWood        ( 233, 194, 166, 255 );
const FColor FColorList::LimeGreen        (  50, 205,  50, 255 );
const FColor FColorList::MandarianOrange  ( 228, 120,  51, 255 );
const FColor FColorList::Maroon           ( 142,  35, 107, 255 );
const FColor FColorList::MediumAquamarine (  50, 205, 153, 255 );
const FColor FColorList::MediumBlue       (  50,  50, 205, 255 );
const FColor FColorList::MediumForestGreen( 107, 142,  35, 255 );
const FColor FColorList::MediumGoldenrod  ( 234, 234, 174, 255 );
const FColor FColorList::MediumOrchid     ( 147, 112, 219, 255 );
const FColor FColorList::MediumSeaGreen   (  66, 111,  66, 255 );
const FColor FColorList::MediumSlateBlue  ( 127,   0, 255, 255 );
const FColor FColorList::MediumSpringGreen( 127, 255,   0, 255 );
const FColor FColorList::MediumTurquoise  ( 112, 219, 219, 255 );
const FColor FColorList::MediumVioletRed  ( 219, 112, 147, 255 );
const FColor FColorList::MediumWood       ( 166, 128, 100, 255 );
const FColor FColorList::MidnightBlue     (  47,  47,  79, 255 );
const FColor FColorList::NavyBlue         (  35,  35, 142, 255 );
const FColor FColorList::NeonBlue         (  77,  77, 255, 255 );
const FColor FColorList::NeonPink         ( 255, 110, 199, 255 );
const FColor FColorList::NewMidnightBlue  (   0,   0, 156, 255 );
const FColor FColorList::NewTan           ( 235, 199, 158, 255 );
const FColor FColorList::OldGold          ( 207, 181,  59, 255 );
const FColor FColorList::Orange           ( 255, 127,   0, 255 );
const FColor FColorList::OrangeRed        ( 255,  36,   0, 255 );
const FColor FColorList::Orchid           ( 219, 112, 219, 255 );
const FColor FColorList::PaleGreen        ( 143, 188, 143, 255 );
const FColor FColorList::Pink             ( 188, 143, 143, 255 );
const FColor FColorList::Plum             ( 234, 173, 234, 255 );
const FColor FColorList::Quartz           ( 217, 217, 243, 255 );
const FColor FColorList::RichBlue         (  89,  89, 171, 255 );
const FColor FColorList::Salmon           ( 111,  66,  66, 255 );
const FColor FColorList::Scarlet          ( 140,  23,  23, 255 );
const FColor FColorList::SeaGreen         (  35, 142, 104, 255 );
const FColor FColorList::SemiSweetChocolate(107,  66,  38, 255 );
const FColor FColorList::Sienna           ( 142, 107,  35, 255 );
const FColor FColorList::Silver           ( 230, 232, 250, 255 );
const FColor FColorList::SkyBlue          (  50, 153, 204, 255 );
const FColor FColorList::SlateBlue        (   0, 127, 255, 255 );
const FColor FColorList::SpicyPink        ( 255,  28, 174, 255 );
const FColor FColorList::SpringGreen      (   0, 255, 127, 255 );
const FColor FColorList::SteelBlue        (  35, 107, 142, 255 );
const FColor FColorList::SummerSky        (  56, 176, 222, 255 );
const FColor FColorList::Tan              ( 219, 147, 112, 255 );
const FColor FColorList::Thistle          ( 216, 191, 216, 255 );
const FColor FColorList::Turquoise        ( 173, 234, 234, 255 );
const FColor FColorList::VeryDarkBrown    (  92,  64,  51, 255 );
const FColor FColorList::VeryLightGrey    ( 205, 205, 205, 255 );
const FColor FColorList::Violet           (  79,  47,  79, 255 );
const FColor FColorList::VioletRed        ( 204,  50, 153, 255 );
const FColor FColorList::Wheat            ( 216, 216, 191, 255 );
const FColor FColorList::YellowGreen      ( 153, 204,  50, 255 );
