// Copyright Epic Games, Inc. All Rights Reserved.

#include "Math/DoubleFloat.h"

constexpr float UE_DF_MIN_PRECISION = 1.0f/(1 << 2);
// Max value of a float before it's precision is lower than UE_DF_MIN_PRECISION
// (there may be 1 more implicit bit available in the significant, but this works as a safe upper bound)
constexpr float UE_DF_FLOAT_MAX_VALUE = ((float)(1 << 23) * UE_DF_MIN_PRECISION - 1.0f);

FMatrix CheckMatrixPrecision(const FMatrix& Matrix)
{
#if !UE_BUILD_SHIPPING
	const double OriginMax = UE_DF_FLOAT_MAX_VALUE;

	const FVector Origin = Matrix.GetOrigin();
	const double OriginX = FMath::Abs(Origin.X);
	const double OriginY = FMath::Abs(Origin.Y);
	const double OriginZ = FMath::Abs(Origin.Z);
	ensureMsgf(OriginX <= OriginMax && OriginY <= OriginMax && OriginZ <= OriginMax, 
		TEXT("Found precision loss while converting matrix to GPU format, verify the input transforms. ")
		TEXT("This error usually indicates the view transform is invalid, or the PreViewTranslation/ViewOrigin was not set up correctly."));
#endif //!UE_BUILD_SHIPPING

	return Matrix;
}

FMatrix44f FDFMatrix::SafeCastMatrix(const FMatrix& Matrix)
{
	return FMatrix44f(CheckMatrixPrecision(Matrix));
}

FMatrix FDFMatrix::MakeToRelativeWorldMatrixDouble(const FVector Origin, const FMatrix& ToWorld)
{
	return CheckMatrixPrecision(ToWorld * FTranslationMatrix(-Origin));
}

FDFMatrix FDFMatrix::MakeToRelativeWorldMatrix(const FVector3f Origin, const FMatrix& ToWorld)
{
	return FDFMatrix(FMatrix44f(MakeToRelativeWorldMatrixDouble(FVector(Origin), ToWorld)), Origin);
}

FMatrix FDFMatrix::MakeClampedToRelativeWorldMatrixDouble(const FVector Origin, const FMatrix& ToWorld)
{
	const double OriginMax = UE_DF_FLOAT_MAX_VALUE;

	// Clamp the relative matrix, avoid allowing the relative translation to get too far away from the origin
	const FVector RelativeOrigin = ToWorld.GetOrigin() - Origin;
	FVector ClampedRelativeOrigin = RelativeOrigin;
	ClampedRelativeOrigin.X = FMath::Clamp(ClampedRelativeOrigin.X, -OriginMax, OriginMax);
	ClampedRelativeOrigin.Y = FMath::Clamp(ClampedRelativeOrigin.Y, -OriginMax, OriginMax);
	ClampedRelativeOrigin.Z = FMath::Clamp(ClampedRelativeOrigin.Z, -OriginMax, OriginMax);

	FMatrix ClampedToRelativeWorld(ToWorld);
	ClampedToRelativeWorld.SetOrigin(ClampedRelativeOrigin);
	return ClampedToRelativeWorld;
}

FDFMatrix FDFMatrix::MakeClampedToRelativeWorldMatrix(const FVector3f Origin, const FMatrix& ToWorld)
{
	return FDFMatrix(FMatrix44f(MakeClampedToRelativeWorldMatrixDouble(FVector(Origin), ToWorld)), Origin);
}

FDFInverseMatrix FDFInverseMatrix::MakeFromRelativeWorldMatrix(const FVector3f Origin, const FMatrix& FromWorld)
{
	return FDFInverseMatrix(FMatrix44f(MakeFromRelativeWorldMatrixDouble(FVector(Origin), FromWorld)), Origin);
}

FMatrix FDFInverseMatrix::MakeFromRelativeWorldMatrixDouble(const FVector Origin, const FMatrix& FromWorld)
{
	return CheckMatrixPrecision(FTranslationMatrix(Origin) * FromWorld);
}
