// Copyright Epic Games, Inc. All Rights Reserved.

#include "Math/Float16.h"

void FFloat16::SetTruncate(float FP32Value)
{

	union
	{
		struct
		{
#if PLATFORM_LITTLE_ENDIAN
			uint16	Mantissa : 10;
			uint16	Exponent : 5;
			uint16	Sign : 1;
#else
			uint16	Sign : 1;
			uint16	Exponent : 5;
			uint16	Mantissa : 10;			
#endif
		} Components;

		uint16	Encoded;
	} FP16;


	FFloat32 FP32(FP32Value);

	// Copy sign-bit
	FP16.Components.Sign = FP32.Components.Sign;

	// Check for zero, denormal or too small value.
	if (FP32.Components.Exponent <= 112)			// Too small exponent? (0+127-15)
	{
		// Set to 0.
		FP16.Components.Exponent = 0;
		FP16.Components.Mantissa = 0;

         // Exponent unbias the single, then bias the halfp
         const int32 NewExp = FP32.Components.Exponent - 127 + 15;
 
         if ( (14 - NewExp) <= 24 ) // Mantissa might be non-zero
         {
             uint32 Mantissa = FP32.Components.Mantissa | 0x800000; // Hidden 1 bit
             FP16.Components.Mantissa = (uint16)(Mantissa >> (14 - NewExp));
			 // Check for rounding
             if ( (Mantissa >> (13 - NewExp)) & 1 ) //-V1051
			 {
                 FP16.Encoded++; // Round, might overflow into exp bit, but this is OK
			 }
         }
	}
	// Check for INF or NaN, or too high value
	else if (FP32.Components.Exponent >= 143)		// Too large exponent? (31+127-15)
	{
		// Set to 65504.0 (max value)
		FP16.Components.Exponent = 30;
		FP16.Components.Mantissa = 1023;
	}
	// Handle normal number.
	else
	{
		FP16.Components.Exponent = uint16(int32(FP32.Components.Exponent) - 127 + 15);
		FP16.Components.Mantissa = uint16(FP32.Components.Mantissa >> 13);
	}

	Encoded = FP16.Encoded;
}


