// Copyright Epic Games, Inc. All Rights Reserved.

#include "Math/FloatPacker.h"

DEFINE_LOG_CATEGORY(LogFloatPacker);
