// Copyright Epic Games, Inc. All Rights Reserved.

#include "Math/GenericOctree.h"

DEFINE_LOG_CATEGORY(LogGenericOctree);
