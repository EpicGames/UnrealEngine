// Copyright Epic Games, Inc. All Rights Reserved.

#include "Math/PackedVector.h"

FFloat3Packed::FFloat3Packed(const FLinearColor& Src)
{
	uint32 IValue[4];
	IValue[0] = *(uint32*)&Src.R;
	IValue[1] = *(uint32*)&Src.G;
	IValue[2] = *(uint32*)&Src.B;
	IValue[3] = *(uint32*)&Src.A;

	uint32 Result[3];

    // X & Y Channels (5-bit exponent, 6-bit mantissa)
    for (uint32 j=0; j < 2; ++j)
    {
        uint32 Sign = IValue[j] & 0x80000000;
        uint32 I = IValue[j] & 0x7FFFFFFF;

        if ((I & 0x7F800000) == 0x7F800000)
        {
            // INF or NAN
            Result[j] = 0x7c0;
            if (( I & 0x7FFFFF ) != 0)
            {
                Result[j] = 0x7c0 | (((I>>17)|(I>>11)|(I>>6)|(I))&0x3f);
            }
            else if ( Sign )
            {
                // -INF is clamped to 0 since 3PK is positive only
                Result[j] = 0;
            }
        }
        else if ( Sign )
        {
            // 3PK is positive only, so clamp to zero
            Result[j] = 0;
        }
        else if (I > 0x477E0000U)
        {
            // The number is too large to be represented as a float11, set to max
            Result[j] = 0x7BF;
        }
        else
        {
            if (I < 0x38800000U)
            {
                // The number is too small to be represented as a normalized float11
                // Convert it to a denormalized value.
                uint32 Shift = 113U - (I >> 23U);
                I = (0x800000U | (I & 0x7FFFFFU)) >> Shift;
            }
            else
            {
                // Rebias the exponent to represent the value as a normalized float11
                I += 0xC8000000U;
            }
     
            Result[j] = ((I + 0xFFFFU + ((I >> 17U) & 1U)) >> 17U)&0x7ffU;
        }
    }

    // Z Channel (5-bit exponent, 5-bit mantissa)
    uint32 Sign = IValue[2] & 0x80000000;
    uint32 I = IValue[2] & 0x7FFFFFFF;

    if ((I & 0x7F800000) == 0x7F800000)
    {
        // INF or NAN
        Result[2] = 0x3e0;
        if ( I & 0x7FFFFF )
        {
            Result[2] = 0x3e0 | (((I>>18)|(I>>13)|(I>>3)|(I))&0x1f);
        }
        else if ( Sign )
        {
            // -INF is clamped to 0 since 3PK is positive only
            Result[2] = 0;
        }
    }
    else if ( Sign )
    {
        // 3PK is positive only, so clamp to zero
        Result[2] = 0;
    }
    else if (I > 0x477C0000U)
    {
        // The number is too large to be represented as a float10, set to max
        Result[2] = 0x3df;
    }
    else
    {
        if (I < 0x38800000U)
        {
            // The number is too small to be represented as a normalized float10
            // Convert it to a denormalized value.
            uint32 Shift = 113U - (I >> 23U);
            I = (0x800000U | (I & 0x7FFFFFU)) >> Shift;
        }
        else
        {
            // Rebias the exponent to represent the value as a normalized float10
            I += 0xC8000000U;
        }
     
        Result[2] = ((I + 0x1FFFFU + ((I >> 18U) & 1U)) >> 18U)&0x3ffU;
    }

    // Pack Result into memory
    v = (Result[0] & 0x7ff)
        | ( (Result[1] & 0x7ff) << 11 )
        | ( (Result[2] & 0x3ff) << 22 );
}

FLinearColor FFloat3Packed::ToLinearColor() const
{
	uint32 Result[4];
    uint32 Mantissa;
    uint32 Exponent;

	const FFloat3Packed* pSource = this;

	// X Channel (6-bit mantissa)
    Mantissa = pSource->xm;

    if ( pSource->xe == 0x1f ) // INF or NAN
    {
        Result[0] = 0x7f800000 | (pSource->xm << 17);
    }
    else
    {
        if ( pSource->xe != 0 ) // The value is normalized
        {
            Exponent = pSource->xe;
        }
        else if (Mantissa != 0) // The value is denormalized
        {
            // Normalize the value in the resulting float
            Exponent = 1;
    
            do
            {
                Exponent--;
                Mantissa <<= 1;
            } while ((Mantissa & 0x40) == 0);
    
            Mantissa &= 0x3F;
        }
        else // The value is zero
        {
            Exponent = (uint32)-112;
        }
    
        Result[0] = ((Exponent + 112) << 23) | (Mantissa << 17);
    }

    // Y Channel (6-bit mantissa)
    Mantissa = pSource->ym;

    if ( pSource->ye == 0x1f ) // INF or NAN
    {
        Result[1] = 0x7f800000 | (pSource->ym << 17);
    }
    else
    {
        if ( pSource->ye != 0 ) // The value is normalized
        {
            Exponent = pSource->ye;
        }
        else if (Mantissa != 0) // The value is denormalized
        {
            // Normalize the value in the resulting float
            Exponent = 1;
    
            do
            {
                Exponent--;
                Mantissa <<= 1;
            } while ((Mantissa & 0x40) == 0);
    
            Mantissa &= 0x3F;
        }
        else // The value is zero
        {
            Exponent = (uint32)-112;
        }
    
        Result[1] = ((Exponent + 112) << 23) | (Mantissa << 17);
    }

    // Z Channel (5-bit mantissa)
    Mantissa = pSource->zm;

    if ( pSource->ze == 0x1f ) // INF or NAN
    {
        Result[2] = 0x7f800000 | (pSource->zm << 17);
    }
    else
    {
        if ( pSource->ze != 0 ) // The value is normalized
        {
            Exponent = pSource->ze;
        }
        else if (Mantissa != 0) // The value is denormalized
        {
            // Normalize the value in the resulting float
            Exponent = 1;
    
            do
            {
                Exponent--;
                Mantissa <<= 1;
            } while ((Mantissa & 0x20) == 0);
    
            Mantissa &= 0x1F;
        }
        else // The value is zero
        {
            Exponent = (uint32)-112;
        }

        Result[2] = ((Exponent + 112) << 23) | (Mantissa << 18);
    }

	FLinearColor ResultColor;
	ResultColor.R = *(float*)&Result[0];
	ResultColor.G = *(float*)&Result[1];
	ResultColor.B = *(float*)&Result[2];
	ResultColor.A = 0;
	return ResultColor;
}

FFixedRGBASigned8::FFixedRGBASigned8(const FLinearColor& Src)
{
	const float Scale = MAX_int8;
	R = (int8)FMath::Clamp<int32>(FMath::RoundToInt(Src.R * Scale), MIN_int8, MAX_int8);
	G = (int8)FMath::Clamp<int32>(FMath::RoundToInt(Src.G * Scale), MIN_int8, MAX_int8);
	B = (int8)FMath::Clamp<int32>(FMath::RoundToInt(Src.B * Scale), MIN_int8, MAX_int8);
	A = (int8)FMath::Clamp<int32>(FMath::RoundToInt(Src.A * Scale), MIN_int8, MAX_int8);
}

FLinearColor FFixedRGBASigned8::ToLinearColor() const
{
	const float Scale = 1.0f / MAX_int8;
	return FLinearColor(R * Scale, G * Scale, B * Scale, A * Scale);
}

FFloat3PackedSE::FFloat3PackedSE(const FLinearColor& Src)
{

	//s=sign, e = exponent, m = mantissa
	//32 bit floating point:
	// s:e:m 1:8:23   
	// if e > 0					float = (-1)^sign X 2^(e-127) X (1.0 + m)
	// if e == 0	&& m!= 0		float = (-1)^sign X 2^(-126) X (0.0 + m)   (subnormal)
	// if e == 0xff && m==0        float = (-1)^sign X infinity					
	// if e == 0xff && m!=0         NaN											
	//
	struct PackedFloat
	{
		union
		{
			struct
			{
				uint32_t m : 23;
				uint32_t e : 8;
				uint32_t s : 1;
			};
			float v;
		};
		bool IsSubnormal() { return (e == 0 && m != 0 ); }
		uint32 Mantissa() { return m + (IsSubnormal() ? 0 : (1 << 23) ); }
		int32 Exponent() { return e - 127; }
		bool IsInfinity() { return (e == 0xff) && (m == 0); }
		bool IsNaN() { return (e == 0xff) && (m != 0); }
	};

	PackedFloat RGBPacked[3];
	RGBPacked[0] = BitCast<PackedFloat, float>(Src.R);
	RGBPacked[1] = BitCast<PackedFloat, float>(Src.G);
	RGBPacked[2] = BitCast<PackedFloat, float>(Src.B);

	uint32 Mantissas[3];
	int32 Exponents[3];

	for (uint32 i = 0; i < 3; ++i)
	{
		Mantissas[i] = RGBPacked[i].Mantissa();
		Exponents[i] = RGBPacked[i].Exponent();

		// 9995 uses (0.0 + m) instead of floating point's (1.0 + m) , so let's fix that here.
		if (!RGBPacked[i].IsSubnormal()) //don't need to fix subnormal (it is already 0.m based)
		{
			// example 2^2 X 1.5 ---> 2^3 X 0.75.  Exponent++, and mantissa/=2
			Exponents[i]++;
			Mantissas[i] /= 2;
		}
		if (  (Exponents[i] < -15 && RGBPacked[i].v != 0.0f) || (RGBPacked[i].v < 0.0f) || RGBPacked[i].IsNaN()) //underflow or negative or NaN
		{
			//As per DirectX implementation, underflow or negative or NaN clamp to a lowest possible exponent and mantissa
			Exponents[i] = -15;
			Mantissas[i] = 0;
		}
		else if (Exponents[i] > 15)  //overflow or infinity
		{
			Exponents[i] = 16; //Match behavior of XMStoreFloat3SE
			Mantissas[i] = 0x7fffff; //as close to 1 as possible
		}
	}

	//exponent now guaranteed to be from -15 to +15
	int32 NewExponent = FMath::Max<int32>(Exponents[0], FMath::Max<int32>(Exponents[1], Exponents[2]));

	for (uint32 i = 0; i < 3; ++i)
	{
		Mantissas[i] = (Mantissas[i] >> ((uint32)(NewExponent - Exponents[i]))); // this can conceivably go to zero if the diff between max and min Exponents > 8 !
		//Mantissa is still 23 bits Need to make it 9 bit
		Mantissas[i]	= Mantissas[i] >> (23 - 9); // now 9 bit
	}
	
	check( Mantissas[0] < (1 << 23) && Mantissas[1] < (1 << 23) && Mantissas[2] < (1 << 23) );

	EncodedValue = (((NewExponent + 15) & 0x01f) << (9 + 9 + 9)) |
				    ((Mantissas[2]      & 0x1ff) << (0 + 9 + 9)) |
				    ((Mantissas[1]      & 0x1ff) << (0 + 0 + 9)) |
				    ((Mantissas[0]      & 0x1ff) << (0 + 0 + 0)) ;

}

FLinearColor FFloat3PackedSE::ToLinearColor() const
{
    int32 SharedExponent8Bits = 0x33800000 + (SharedExponent << 23);
    float Scale = BitCast<float, int32>(SharedExponent8Bits);

	return FLinearColor( Scale * float(RMantissa), Scale * float(GMantissa), Scale * float(BMantissa),1.0f );
}
