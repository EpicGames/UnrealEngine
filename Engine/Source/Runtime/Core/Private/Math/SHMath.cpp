// Copyright Epic Games, Inc. All Rights Reserved.


#include "Math/SHMath.h"

#include "ColorManagement/ColorSpace.h"
#include "HAL/IConsoleManager.h"

//
//	Spherical harmonic globals.
//
float NormalizationConstants[9];
int32 BasisL[9];
int32 BasisM[9];

/** Computes a factorial. */
static int32 Factorial(int32 A)
{
	if(A == 0)
	{
		return 1;
	}
	else
	{
		return A * Factorial(A - 1);
	}
}

/** Initializes the tables used to calculate SH values. */
static int32 InitSHTables()
{
	int32	L = 0,
		M = 0;

	for(int32 BasisIndex = 0;BasisIndex < 9;BasisIndex++)
	{
		BasisL[BasisIndex] = L;
		BasisM[BasisIndex] = M;

		NormalizationConstants[BasisIndex] = FMath::Sqrt(
			(float(2 * L + 1) / float(4 * UE_PI)) *
			(float(Factorial(L - FMath::Abs(M))) / float(Factorial(L + FMath::Abs(M))))
			);

		if(M != 0)
			NormalizationConstants[BasisIndex] *= FMath::Sqrt(2.f);

		M++;
		if(M > L)
		{
			L++;
			M = -L;
		}
	}

	return 0;
}
static int32 InitDummy = InitSHTables();

/** So that e.g. LP(1,1,1) which evaluates to -sqrt(1-1^2) is 0.*/
FORCEINLINE float SafeSqrt(float F)
{
	return FMath::Abs(F) > UE_KINDA_SMALL_NUMBER ? FMath::Sqrt(F) : 0.f;
}

/** Evaluates the LegendrePolynomial for L,M at X */
float LegendrePolynomial(int32 L,int32 M,float X)
{
	switch(L)
	{
	case 0:
		return 1;
	case 1:
		if(M == 0)
			return X;
		else if(M == 1)
			return -SafeSqrt(1 - X * X);
		break;
	case 2:
		if(M == 0)
			return -0.5f + (3 * X * X) / 2;
		else if(M == 1)
			return -3 * X * SafeSqrt(1 - X * X);
		else if(M == 2)
			return -3 * (-1 + X * X);
		break;
	case 3:
		if(M == 0)
			return -(3 * X) / 2 + (5 * X * X * X) / 2;
		else if(M == 1)
			return -3 * SafeSqrt(1 - X * X) / 2 * (-1 + 5 * X * X);
		else if(M == 2)
			return -15 * (-X + X * X * X);
		else if(M == 3)
			return -15 * FMath::Pow(1 - X * X,1.5f);
		break;
	case 4:
		if(M == 0)
			return 0.125f * (3.0f - 30.0f * X * X + 35.0f * X * X * X * X);
		else if(M == 1)
			return -2.5f * X * SafeSqrt(1.0f - X * X) * (7.0f * X * X - 3.0f);
		else if(M == 2)
			return -7.5f * (1.0f - 8.0f * X * X + 7.0f * X * X * X * X);
		else if(M == 3)
			return -105.0f * X * FMath::Pow(1 - X * X,1.5f);
		else if(M == 4)
			return 105.0f * FMath::Square(X * X - 1.0f);
		break;
	case 5:
		if(M == 0)
			return 0.125f * X * (15.0f - 70.0f * X * X + 63.0f * X * X * X * X);
		else if(M == 1)
			return -1.875f * SafeSqrt(1.0f - X * X) * (1.0f - 14.0f * X * X + 21.0f * X * X * X * X);
		else if(M == 2)
			return -52.5f * (X - 4.0f * X * X * X + 3.0f * X * X * X * X * X);
		else if(M == 3)
			return -52.5f * FMath::Pow(1.0f - X * X,1.5f) * (9.0f * X * X - 1.0f);
		else if(M == 4)
			return 945.0f * X * FMath::Square(X * X - 1);
		else if(M == 5)
			return -945.0f * FMath::Pow(1.0f - X * X,2.5f);
		break;
	};

	return 0.0f;
}

FLinearColor SHGetLuminanceFactors()
{
	static const auto CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.LegacyLuminanceFactors"));

	if (CVar && CVar->GetInt() != 0)
	{
		return FLinearColor(0.3f, 0.59f, 0.11f);
	}
	else
	{
		return UE::Color::FColorSpace::GetWorking().GetLuminanceFactors();
	}
}
