// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Sphere.cpp: Implements the FSphere class.
=============================================================================*/

#include "Math/Sphere.h"

namespace UE {
namespace Math {

// [ Ritter 1990, "An Efficient Bounding Sphere" ]
template<typename T>
static FORCEINLINE void ConstructFromPoints(TSphere<T>& ThisSphere, const TVector<T>* Points, int32 Count)
{
	check(Count > 0);

	// Min/max points of AABB
	int32 MinIndex[3] = { 0, 0, 0 };
	int32 MaxIndex[3] = { 0, 0, 0 };

	for (int i = 0; i < Count; i++)
	{
		for (int k = 0; k < 3; k++)
		{
			MinIndex[k] = Points[i][k] < Points[MinIndex[k]][k] ? i : MinIndex[k];
			MaxIndex[k] = Points[i][k] > Points[MaxIndex[k]][k] ? i : MaxIndex[k];
		}
	}

	T LargestDistSqr = 0.0f;
	int32 LargestAxis = 0;
	for (int k = 0; k < 3; k++)
	{
		TVector<T> PointMin = Points[MinIndex[k]];
		TVector<T> PointMax = Points[MaxIndex[k]];

		T DistSqr = (PointMax - PointMin).SizeSquared();
		if (DistSqr > LargestDistSqr)
		{
			LargestDistSqr = DistSqr;
			LargestAxis = k;
		}
	}

	TVector<T> PointMin = Points[MinIndex[LargestAxis]];
	TVector<T> PointMax = Points[MaxIndex[LargestAxis]];

	ThisSphere.Center = 0.5f * (PointMin + PointMax);
	ThisSphere.W = 0.5f * FMath::Sqrt(LargestDistSqr);
	T WSqr = ThisSphere.W * ThisSphere.W;

	// Adjust to fit all points
	for (int i = 0; i < Count; i++)
	{
		T DistSqr = (Points[i] - ThisSphere.Center).SizeSquared();

		if (DistSqr > WSqr)
		{
			T Dist = FMath::Sqrt(DistSqr);
			T t = 0.5f + 0.5f * (ThisSphere.W / Dist);

			ThisSphere.Center = FMath::LerpStable(Points[i], ThisSphere.Center, t);
			ThisSphere.W = 0.5f * (ThisSphere.W + Dist);
		}
	}
}

template<> TSphere<float>::TSphere(const TVector<float>* Points, int32 Count)
{
	ConstructFromPoints(*this, Points, Count);
}

template<> TSphere<double>::TSphere(const TVector<double>* Points, int32 Count)
{
	ConstructFromPoints(*this, Points, Count);
}


template<typename T>
static FORCEINLINE void ConstructFromSpheres(TSphere<T>& ThisSphere, const TSphere<T>* Spheres, int32 Count)
{
	check(Count > 0);

	// Min/max points of AABB
	int32 MinIndex[3] = { 0, 0, 0 };
	int32 MaxIndex[3] = { 0, 0, 0 };

	for (int i = 0; i < Count; i++)
	{
		for (int k = 0; k < 3; k++)
		{
			MinIndex[k] = Spheres[i].Center[k] - Spheres[i].W < Spheres[MinIndex[k]].Center[k] - Spheres[MinIndex[k]].W ? i : MinIndex[k];
			MaxIndex[k] = Spheres[i].Center[k] + Spheres[i].W > Spheres[MaxIndex[k]].Center[k] + Spheres[MaxIndex[k]].W ? i : MaxIndex[k];
		}
	}

	T LargestDist = 0.0f;
	int32 LargestAxis = 0;
	for (int k = 0; k < 3; k++)
	{
		TSphere<T> SphereMin = Spheres[MinIndex[k]];
		TSphere<T> SphereMax = Spheres[MaxIndex[k]];

		T Dist = (SphereMax.Center - SphereMin.Center).Size() + SphereMin.W + SphereMax.W;
		if (Dist > LargestDist)
		{
			LargestDist = Dist;
			LargestAxis = k;
		}
	}

	ThisSphere = Spheres[MinIndex[LargestAxis]];
	ThisSphere += Spheres[MaxIndex[LargestAxis]];

	// Adjust to fit all spheres
	for (int i = 0; i < Count; i++)
	{
		ThisSphere += Spheres[i];
	}
}

template<> TSphere<float>::TSphere(const TSphere<float>* Spheres, int32 Count)
{
	ConstructFromSpheres(*this, Spheres, Count);
}

template<> TSphere<double>::TSphere(const TSphere<double>* Spheres, int32 Count)
{
	ConstructFromSpheres(*this, Spheres, Count);
}


}
}