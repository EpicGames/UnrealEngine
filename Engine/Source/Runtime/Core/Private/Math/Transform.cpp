// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Transform.cpp
=============================================================================*/

#include "Math/Transform.h"

#if !ENABLE_VECTORIZED_TRANSFORM

#include "Misc/DefaultValueHelper.h"

DEFINE_LOG_CATEGORY_STATIC(LogTransform, Log, All);

namespace UE
{
namespace Math
{

template<> const FTransform3f FTransform3f::Identity(FQuat4f(0.f, 0.f, 0.f, 1.f), FVector3f(0.f), FVector3f(1.f));
template<> const FTransform3d FTransform3d::Identity(FQuat4d(0.0, 0.0, 0.0, 1.0), FVector3d(0.0), FVector3d(1.0));

// Replacement of Inverse of TMatrix<T>

/**
* Does a debugf of the contents of this BoneAtom.
*/
template<typename T>
void TTransform<T>::DebugPrint() const
{
	UE_LOGF(LogTransform, Log, "%ls", *ToHumanReadableString());
}

template<typename T>
FString TTransform<T>::ToHumanReadableString() const
{
	TRotator<T> R(GetRotation());
	TVector<T> TT(GetTranslation());
	TVector<T> S(GetScale3D());

	FString Output = FString::Printf(TEXT("Rotation: Pitch %f Yaw %f Roll %f\r\n"), R.Pitch, R.Yaw, R.Roll);
	Output += FString::Printf(TEXT("Translation: %f %f %f\r\n"), TT.X, TT.Y, TT.Z);
	Output += FString::Printf(TEXT("Scale3D: %f %f %f\r\n"), S.X, S.Y, S.Z);

	return Output;
}


template<typename T>
FString TTransform<T>::ToString() const
{
	const TRotator<T> R(Rotator());
	const TVector<T> TT(GetTranslation());
	const TVector<T> S(GetScale3D());

	return FString::Printf(TEXT("%f,%f,%f|%f,%f,%f|%f,%f,%f"), TT.X, TT.Y, TT.Z, R.Pitch, R.Yaw, R.Roll, S.X, S.Y, S.Z);
}

template<typename T>
bool TTransform<T>::InitFromString( const FString& Source )
{
	TArray<FString> ComponentStrings;
	Source.ParseIntoArray(ComponentStrings, TEXT("|"), true);
	const int32 NumComponents = ComponentStrings.Num();
	if(3 != NumComponents)
	{
		return false;
	}

	// Translation
	TVector<T> ParsedTranslation = TVector<T>::ZeroVector;
	if( !FDefaultValueHelper::ParseVector(ComponentStrings[0], ParsedTranslation) )
	{
		return false;
	}

	// Rotation
	TRotator<T> ParsedRotation = TRotator<T>::ZeroRotator;
	if( !FDefaultValueHelper::ParseRotator(ComponentStrings[1], ParsedRotation) )
	{
		return false;
	}

	// Scale
	TVector<T> ParsedScale = TVector<T>::OneVector;
	if( !FDefaultValueHelper::ParseVector(ComponentStrings[2], ParsedScale) )
	{
		return false;
	}

	SetComponents(TQuat<T>(ParsedRotation), ParsedTranslation, ParsedScale);

	return true;
}

#define DEBUG_INVERSE_TRANSFORM 0
template<typename T>
TTransform<T> TTransform<T>::GetRelativeTransformReverse(const TTransform<T>& Other) const
{
	// A(-1) * B = VQS(B) * VQS(A)(-1)
	// 
	// Scale = S(B) / S(A)
	// Rotation = Q(B) * Q(A)(-1)
	// Translation = T(B) - [Q(B) * Q(A)(-1) * S(B)/S(A) * T(A) * Q(A) * Q(B)(-1)]
	// where A = this, and B = Other
	TTransform<T> Result;

	TVector<T> SafeRecipScale3D = GetSafeScaleReciprocal(Scale3D);
	Result.Scale3D = Other.Scale3D * SafeRecipScale3D;

	Result.Rotation = Other.Rotation * Rotation.Inverse();

	Result.Translation = Other.Translation - (Result.Rotation * (Result.Scale3D * Translation));

#if DEBUG_INVERSE_TRANSFORM
	TMatrix<T> AM = ToMatrixWithScale();
	TMatrix<T> BM = Other.ToMatrixWithScale();

	Result.DebugEqualMatrix(AM.InverseFast() * BM);
#endif

	return Result;
}

/**
 * Set current transform and the relative to ParentTransform.
 * Equates to This = This->GetRelativeTransform(Parent), but saves the intermediate TTransform<T> storage and copy.
 */
template<typename T>
void TTransform<T>::SetToRelativeTransform(const TTransform<T>& ParentTransform)
{
	// A * B(-1) = VQS(B)(-1) (VQS (A))
	// 
	// Scale = S(A)/S(B)
	// Rotation = Q(B)(-1) * Q(A)
	// Translation = 1/S(B) *[Q(B)(-1)*(T(A)-T(B))*Q(B)]
	// where A = this, B = Other
#if DEBUG_INVERSE_TRANSFORM
 	TMatrix<T> AM = ToMatrixWithScale();
 	TMatrix<T> BM = ParentTransform.ToMatrixWithScale();
#endif

	const TVector<T> SafeRecipScale3D = GetSafeScaleReciprocal(ParentTransform.Scale3D, UE_SMALL_NUMBER);
	const TQuat<T> InverseRot = ParentTransform.Rotation.Inverse();

	Scale3D *= SafeRecipScale3D;	
	Translation = (InverseRot * (Translation - ParentTransform.Translation)) * SafeRecipScale3D;
	Rotation = InverseRot * Rotation;

#if DEBUG_INVERSE_TRANSFORM
 	DebugEqualMatrix(AM *  BM.InverseFast());
#endif
}

template<typename T>
void TTransform<T>::GetRelativeTransformUsingMatrixWithScale(TTransform<T>* OutTransform, const TTransform<T>* Base, const TTransform<T>* Relative)
{
	// the goal of using M is to get the correct orientation
	// but for translation, we still need scale
	TMatrix<T> AM = Base->ToMatrixWithScale();
	TMatrix<T> BM = Relative->ToMatrixWithScale();
	// get combined scale
	TVector<T> SafeRecipScale3D = GetSafeScaleReciprocal(Relative->Scale3D, UE_SMALL_NUMBER);
	TVector<T> DesiredScale3D = Base->Scale3D*SafeRecipScale3D;
	ConstructTransformFromMatrixWithDesiredScale(AM, BM.Inverse(), DesiredScale3D, *OutTransform);
}

template<typename T>
TTransform<T> TTransform<T>::GetRelativeTransform(const TTransform<T>& Other) const
{
	// A * B(-1) = VQS(B)(-1) (VQS (A))
	// 
	// Scale = S(A)/S(B)
	// Rotation = Q(B)(-1) * Q(A)
	// Translation = 1/S(B) *[Q(B)(-1)*(T(A)-T(B))*Q(B)]
	// where A = this, B = Other
	TTransform<T> Result;

	if (AnyHasNegativeScale(Scale3D, Other.GetScale3D()))
	{
		// @note, if you have 0 scale with negative, you're going to lose rotation as it can't convert back to quat
		GetRelativeTransformUsingMatrixWithScale(&Result, this, &Other);
	}
	else
	{
		TVector<T> SafeRecipScale3D = GetSafeScaleReciprocal(Other.Scale3D, UE_SMALL_NUMBER);
		Result.Scale3D = Scale3D*SafeRecipScale3D;

		if (Other.Rotation.IsNormalized() == false)
		{
			return TTransform<T>::Identity;
		}

		TQuat<T> Inverse = Other.Rotation.Inverse();
		Result.Rotation = Inverse*Rotation;

		Result.Translation = (Inverse*(Translation - Other.Translation))*(SafeRecipScale3D);

#if DEBUG_INVERSE_TRANSFORM
		TMatrix<T> AM = ToMatrixWithScale();
		TMatrix<T> BM = Other.ToMatrixWithScale();

		Result.DebugEqualMatrix(AM *  BM.InverseFast());

#endif
	}

	return Result;
}

template<typename T>
bool TTransform<T>::DebugEqualMatrix(const TMatrix<T>& Matrix) const
{
	TTransform<T> TestResult(Matrix);
	if (!Equals(TestResult))
	{
		// see now which one isn't equal
		if (!Scale3D.Equals(TestResult.Scale3D, 0.01f))
		{
			UE_LOGF(LogTransform, Log, "Matrix(S)\t%ls", *TestResult.Scale3D.ToString());
			UE_LOGF(LogTransform, Log, "VQS(S)\t%ls", *Scale3D.ToString());
		}

		// see now which one isn't equal
		if (!Rotation.Equals(TestResult.Rotation))
		{
			UE_LOGF(LogTransform, Log, "Matrix(R)\t%ls", *TestResult.Rotation.ToString());
			UE_LOGF(LogTransform, Log, "VQS(R)\t%ls", *Rotation.ToString());
		}

		// see now which one isn't equal
		if (!Translation.Equals(TestResult.Translation, 0.01f))
		{
			UE_LOGF(LogTransform, Log, "Matrix(T)\t%ls", *TestResult.Translation.ToString());
			UE_LOGF(LogTransform, Log, "VQS(T)\t%ls", *Translation.ToString());
		}
		return false;
	}

	return true;
}


} // namespace UE::Math
} // namespace UE

// Instantiate for linker.
template struct UE::Math::TTransform<float>;
template struct UE::Math::TTransform<double>;

#endif // #if !ENABLE_VECTORIZED_TRANSFORM
