// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Transform.cpp
=============================================================================*/

#include "Math/TransformVectorized.h"
#include "Math/Transform.h"
#include "Misc/AssertionMacros.h"
#include "Math/UnrealMathUtility.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Logging/LogMacros.h"
#include "Math/MathFwd.h"

#if ENABLE_VECTORIZED_TRANSFORM

#include "Misc/DefaultValueHelper.h"

DEFINE_LOG_CATEGORY_STATIC(LogTransform, Log, All);

namespace UE
{
namespace Math
{

template<> const FTransform3f FTransform3f::Identity(FQuat4f(0.f, 0.f, 0.f, 1.f), FVector3f(0.f), FVector3f(1.f));
template<> const FTransform3d FTransform3d::Identity(FQuat4d(0.0, 0.0, 0.0, 1.0), FVector3d(0.0), FVector3d(1.0));

// Replacement of Inverse of TMatrix<T>

/**
* Does a debugf of the contents of this BoneAtom.
*/
template<typename T>
void TTransform<T>::DebugPrint() const
{
	UE_LOGF(LogTransform, Log, "%ls", *ToHumanReadableString());
}

template<typename T>
FString TTransform<T>::ToHumanReadableString() const
{
	TRotator<T> R(GetRotation());
	TVector<T> Tr(GetTranslation());
	TVector<T> S(GetScale3D());

	FString Output= FString::Printf(TEXT("Rotation: Pitch %f Yaw %f Roll %f\r\n"), R.Pitch, R.Yaw, R.Roll);
	Output += FString::Printf(TEXT("Translation: %f %f %f\r\n"), Tr.X, Tr.Y, Tr.Z);
	Output += FString::Printf(TEXT("Scale3D: %f %f %f\r\n"), S.X, S.Y, S.Z);

	return Output;
}


template<typename T>
FString TTransform<T>::ToString() const
{
	const TRotator<T> R(Rotator());
	const TVector<T> Tr(GetTranslation());
	const TVector<T> S(GetScale3D());

	return FString::Printf(TEXT("%f,%f,%f|%f,%f,%f|%f,%f,%f"), Tr.X, Tr.Y, Tr.Z, R.Pitch, R.Yaw, R.Roll, S.X, S.Y, S.Z);
}

template<typename T>
bool TTransform<T>::InitFromString( const FString& Source )
{
	TArray<FString> ComponentStrings;
	Source.ParseIntoArray(ComponentStrings, TEXT("|"), true);
	const int32 NumComponents = ComponentStrings.Num();
	if(3 != NumComponents)
	{
		return false;
	}

	// Translation
	TVector<T> ParsedTranslation = TVector<T>::ZeroVector;
	if( !FDefaultValueHelper::ParseVector(ComponentStrings[0], ParsedTranslation) )
	{
		return false;
	}

	// Rotation
	TRotator<T> ParsedRotation = TRotator<T>::ZeroRotator;
	if( !FDefaultValueHelper::ParseRotator(ComponentStrings[1], ParsedRotation) )
	{
		return false;
	}

	// Scale
	TVector<T> Scale = TVector<T>::OneVector;
	if( !FDefaultValueHelper::ParseVector(ComponentStrings[2], Scale) )
	{
		return false;
	}

	SetComponents(TQuat<T>::MakeFromRotator(ParsedRotation), ParsedTranslation, Scale);

	return true;
}

#define DEBUG_INVERSE_TRANSFORM 0
template<typename T>
TTransform<T> TTransform<T>::GetRelativeTransformReverse(const TTransform<T>& Other) const
{
	// A(-1) * B = VQS(B) * VQS(A)(-1)
	// 
	// Scale = S(B) / S(A)
	// Rotation = Q(B) * Q(A)(-1)
	// Translation = T(B) - [Q(B) * Q(A)(-1) * S(B)/S(A) * T(A) * Q(A) * Q(B)(-1)]
	// where A = this, and B = Other
	TTransform<T> Result;

	// Scale = S(B) / S(A)
	TransformVectorRegister VSafeScale3D = VectorSet_W0(GetSafeScaleReciprocal(Scale3D));
	TransformVectorRegister VScale3D = VectorMultiply(Other.Scale3D, VSafeScale3D);

	// Rotation = Q(B) * Q(A)(-1)
	TransformVectorRegister VInverseRot = VectorQuaternionInverse(Rotation);
	TransformVectorRegister VRotation = VectorQuaternionMultiply2(Other.Rotation, VInverseRot);

	// RotatedScaledTranslation
	TransformVectorRegister VRS = VectorQuaternionRotateVector(VRotation, VectorMultiply(VScale3D, Translation));

	// Translation = T(B) - [Q(B) * Q(A)(-1) * S(B)/S(A) * T(A) * Q(A) * Q(B)(-1)]
	TransformVectorRegister VTranslation = VectorSet_W0(VectorSubtract(Other.Translation, VRS));

	Result.Scale3D = VScale3D;
	Result.Translation = VTranslation;
	Result.Rotation = VRotation;
		
	Result.DiagnosticCheckNaN_All();

#if DEBUG_INVERSE_TRANSFORM
	TMatrix<T> AM = ToMatrixWithScale();
	TMatrix<T> BM = Other.ToMatrixWithScale();

	Result.DebugEqualMatrix(AM.InverseFast() * BM);
#endif

	return Result;
}

/**
 * Set current transform and the relative to ParentTransform.
 * Equates to This = This->GetRelativeTransform(Parent), but saves the intermediate TTransform<T> storage and copy.
 */
template<typename T>
void TTransform<T>::SetToRelativeTransform(const TTransform<T>& ParentTransform)
{
	// A * B(-1) = VQS(B)(-1) (VQS (A))
	// 
	// Scale = S(A)/S(B)
	// Rotation = Q(B)(-1) * Q(A)
	// Translation = 1/S(B) *[Q(B)(-1)*(T(A)-T(B))*Q(B)]
	// where A = this, B = Other
#if DEBUG_INVERSE_TRANSFORM
	TMatrix<T> AM = ToMatrixWithScale();
	TMatrix<T> BM = ParentTransform.ToMatrixWithScale();
#endif
	
	checkSlow(ParentTransform.IsRotationNormalized());

	// Scale = S(A)/S(B)	
	TransformVectorRegister VSafeScale3D	= VectorSet_W0(GetSafeScaleReciprocal(ParentTransform.Scale3D, ScalarRegister(UE_SMALL_NUMBER)));
	Scale3D = VectorMultiply(Scale3D, VSafeScale3D);
	
	//VQTranslation = (  ( T(A).X - T(B).X ),  ( T(A).Y - T(B).Y ), ( T(A).Z - T(B).Z), 0.f );
	TransformVectorRegister VQTranslation = VectorSet_W0(VectorSubtract(Translation, ParentTransform.Translation));

	// Inverse RotatedTranslation
	TransformVectorRegister VInverseParentRot = VectorQuaternionInverse(ParentTransform.Rotation);
	TransformVectorRegister VR = VectorQuaternionRotateVector(VInverseParentRot, VQTranslation);

	// Translation = 1/S(B)
	Translation = VectorMultiply(VR, VSafeScale3D);

	// Rotation = Q(B)(-1) * Q(A)	
	Rotation = VectorQuaternionMultiply2(VInverseParentRot, Rotation);

	DiagnosticCheckNaN_All(); 

#if DEBUG_INVERSE_TRANSFORM
	DebugEqualMatrix(AM *  BM.InverseFast());
#endif
}

template<typename T>
void TTransform<T>::GetRelativeTransformUsingMatrixWithScale(TTransform<T>* OutTransform, const TTransform<T>* Base, const TTransform<T>* Relative)
{
	// the goal of using M is to get the correct orientation
	// but for translation, we still need scale
	TMatrix<T> AM = Base->ToMatrixWithScale();
	TMatrix<T> BM = Relative->ToMatrixWithScale();
	// get combined scale
	// Scale = S(A)/S(B)
	static ScalarRegister STolerance(UE_SMALL_NUMBER);
	TransformVectorRegister VSafeScale3D = VectorSet_W0(GetSafeScaleReciprocal(Relative->Scale3D, STolerance));
	TransformVectorRegister VScale3D = VectorMultiply(Base->Scale3D, VSafeScale3D);
	ConstructTransformFromMatrixWithDesiredScale(AM, BM.Inverse(), VScale3D, *OutTransform);
}

template<typename T>
TTransform<T> TTransform<T>::GetRelativeTransform(const TTransform<T>& Other) const
{
	// A * B(-1) = VQS(B)(-1) (VQS (A))
	// 
	// Scale = S(A)/S(B)
	// Rotation = Q(B)(-1) * Q(A)
	// Translation = 1/S(B) *[Q(B)(-1)*(T(A)-T(B))*Q(B)]
	// where A = this, B = Other
	TTransform<T> Result;
		
	if (Other.IsRotationNormalized() == false)
	{
		return TTransform<T>::Identity;
	}

	if (Private_AnyHasNegativeScale(this->Scale3D, Other.Scale3D))
	{
		// @note, if you have 0 scale with negative, you're going to lose rotation as it can't convert back to quat
		GetRelativeTransformUsingMatrixWithScale(&Result, this, &Other);
	}
	else
	{
		// Scale = S(A)/S(B)
		static ScalarRegister STolerance(UE_SMALL_NUMBER);
		TransformVectorRegister VSafeScale3D = VectorSet_W0(GetSafeScaleReciprocal(Other.Scale3D, STolerance));

		TransformVectorRegister VScale3D = VectorMultiply(Scale3D, VSafeScale3D);

		//VQTranslation = (  ( T(A).X - T(B).X ),  ( T(A).Y - T(B).Y ), ( T(A).Z - T(B).Z), 0.f );
		TransformVectorRegister VQTranslation = VectorSet_W0(VectorSubtract(Translation, Other.Translation));

		// Inverse RotatedTranslation
		TransformVectorRegister VInverseRot = VectorQuaternionInverse(Other.Rotation);
		TransformVectorRegister VR = VectorQuaternionRotateVector(VInverseRot, VQTranslation);

		//Translation = 1/S(B)
		TransformVectorRegister VTranslation = VectorMultiply(VR, VSafeScale3D);

		// Rotation = Q(B)(-1) * Q(A)	
		TransformVectorRegister VRotation = VectorQuaternionMultiply2(VInverseRot, Rotation);

		Result.Scale3D = VScale3D;
		Result.Translation = VTranslation;
		Result.Rotation = VRotation;

		Result.DiagnosticCheckNaN_All();
#if DEBUG_INVERSE_TRANSFORM
		TMatrix<T> AM = ToMatrixWithScale();
		TMatrix<T> BM = Other.ToMatrixWithScale();

		Result.DebugEqualMatrix(AM *  BM.InverseFast());
#endif
	}

	return Result;
}

template<typename T>
bool TTransform<T>::DebugEqualMatrix(const TMatrix<T>& Matrix) const
{
	TTransform<T> TestResult(Matrix);
	if (!Equals(TestResult))
	{
		// see now which one isn't equal		
		if (!Scale3DEquals(TestResult, 0.01f))
		{
			UE_LOGF(LogTransform, Log, "Matrix(S)\t%ls", *TestResult.GetScale3D().ToString());
			UE_LOGF(LogTransform, Log, "VQS(S)\t%ls", *GetScale3D().ToString());
		}

		// see now which one isn't equal
		if (!RotationEquals(TestResult))
		{
			UE_LOGF(LogTransform, Log, "Matrix(R)\t%ls", *TestResult.GetRotation().ToString());
			UE_LOGF(LogTransform, Log, "VQS(R)\t%ls", *GetRotation().ToString());
		}

		// see now which one isn't equal
		if (!TranslationEquals(TestResult, 0.01f))
		{
			UE_LOGF(LogTransform, Log, "Matrix(T)\t%ls", *TestResult.GetTranslation().ToString());
			UE_LOGF(LogTransform, Log, "VQS(T)\t%ls", *GetTranslation().ToString());
		}
		return false;
	}

	return true;
}

template<typename T>
void TTransform<T>::Multiply(TTransform<T>* OutTransform, const TTransform<T>* A, const TTransform<T>* B)
{
	A->DiagnosticCheckNaN_All();
	B->DiagnosticCheckNaN_All();

	checkSlow(A->IsRotationNormalized());
	checkSlow(B->IsRotationNormalized());

	//	When Q = quaternion, S = single scalar scale, and T = translation
	//	QST(A) = Q(A), S(A), T(A), and QST(B) = Q(B), S(B), T(B)

	//	QST (AxB) 

	// QST(A) = Q(A)*S(A)*P*-Q(A) + T(A)
	// QST(AxB) = Q(B)*S(B)*QST(A)*-Q(B) + T(B)
	// QST(AxB) = Q(B)*S(B)*[Q(A)*S(A)*P*-Q(A) + T(A)]*-Q(B) + T(B)
	// QST(AxB) = Q(B)*S(B)*Q(A)*S(A)*P*-Q(A)*-Q(B) + Q(B)*S(B)*T(A)*-Q(B) + T(B)
	// QST(AxB) = [Q(B)*Q(A)]*[S(B)*S(A)]*P*-[Q(B)*Q(A)] + Q(B)*S(B)*T(A)*-Q(B) + T(B)

	//	Q(AxB) = Q(B)*Q(A)
	//	S(AxB) = S(A)*S(B)
	//	T(AxB) = Q(B)*S(B)*T(A)*-Q(B) + T(B)
	checkSlow(VectorGetComponent(A->Scale3D, 3) == 0.f);
	checkSlow(VectorGetComponent(B->Scale3D, 3) == 0.f);

	if (Private_AnyHasNegativeScale(A->Scale3D, B->Scale3D))
	{
		// @note, if you have 0 scale with negative, you're going to lose rotation as it can't convert back to quat
		MultiplyUsingMatrixWithScale(OutTransform, A, B);
	}
	else
	{
		const TransformVectorRegister QuatA = A->Rotation;
		const TransformVectorRegister QuatB = B->Rotation;
		const TransformVectorRegister TranslateA = A->Translation;
		const TransformVectorRegister TranslateB = B->Translation;
		const TransformVectorRegister ScaleA = A->Scale3D;
		const TransformVectorRegister ScaleB = B->Scale3D;

		// RotationResult = B.Rotation * A.Rotation
		OutTransform->Rotation = VectorQuaternionMultiply2(QuatB, QuatA);

		// TranslateResult = B.Rotate(B.Scale * A.Translation) + B.Translate
		const TransformVectorRegister ScaledTransA = VectorMultiply(TranslateA, ScaleB);
		const TransformVectorRegister RotatedTranslate = VectorQuaternionRotateVector(QuatB, ScaledTransA);
		OutTransform->Translation = VectorAdd(RotatedTranslate, TranslateB);

		// ScaleResult = Scale.B * Scale.A
		OutTransform->Scale3D = VectorMultiply(ScaleA, ScaleB);
	}
}

} // namespace UE::Math
} // namespace UE


// Instantiate for linker.
template struct UE::Math::TTransform<float>;
template struct UE::Math::TTransform<double>;

#endif // ENABLE_VECTORIZED_TRANSFORM
