// Copyright Epic Games, Inc. All Rights Reserved.

#include "Math/UnitConversion.h"

#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Math/BasicMathExpressionEvaluator.h"
#include "Math/UnitConversion.inl"
#include "Math/UnrealMathUtility.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CString.h"
#include "Misc/ExpressionParser.h"
#include "Misc/Guid.h"
#include "Templates/UniquePtr.h"
#include "Templates/UnrealTemplate.h"

PRAGMA_DISABLE_UNSAFE_TYPECAST_WARNINGS

#define LOCTEXT_NAMESPACE "UnitConversion"

/** Structure used to match units when parsing */
struct FParseCandidate
{
	const TCHAR* String;
	EUnit Unit;
};

//TODO: Add note about updating
constexpr FParseCandidate ParseCandidates[] = {
	
	{ TEXT("Micrometers"),			EUnit::Micrometers },			{ TEXT("um"),		EUnit::Micrometers }, 			{ TEXT("\u00B5m"),	EUnit::Micrometers },
	{ TEXT("Millimeters"),			EUnit::Millimeters },			{ TEXT("mm"),		EUnit::Millimeters },
	{ TEXT("Centimeters"),			EUnit::Centimeters },			{ TEXT("cm"),		EUnit::Centimeters },
	{ TEXT("Meters"),				EUnit::Meters },				{ TEXT("m"),		EUnit::Meters },
	{ TEXT("Kilometers"),			EUnit::Kilometers },			{ TEXT("km"),		EUnit::Kilometers },
	{ TEXT("Inches"),				EUnit::Inches },				{ TEXT("in"),		EUnit::Inches },
	{ TEXT("Feet"),					EUnit::Feet },					{ TEXT("ft"),		EUnit::Feet },
	{ TEXT("Yards"),				EUnit::Yards },					{ TEXT("yd"),		EUnit::Yards },
	{ TEXT("Miles"),				EUnit::Miles },					{ TEXT("mi"),		EUnit::Miles },
	{ TEXT("Lightyears"),			EUnit::Lightyears },			{ TEXT("ly"),		EUnit::Lightyears },

	{ TEXT("Degrees"),				EUnit::Degrees },				{ TEXT("deg"),		EUnit::Degrees },				{ TEXT("\u00B0"),	EUnit::Degrees },
	{ TEXT("Radians"),				EUnit::Radians },				{ TEXT("rad"),		EUnit::Radians },
		
	{ TEXT("CentimetersPerSecond"),	EUnit::CentimetersPerSecond },	{ TEXT("cm/s"),		EUnit::CentimetersPerSecond },
	{ TEXT("MetersPerSecond"),		EUnit::MetersPerSecond },		{ TEXT("m/s"),		EUnit::MetersPerSecond },
	{ TEXT("KilometersPerHour"),	EUnit::KilometersPerHour },		{ TEXT("km/h"),		EUnit::KilometersPerHour },		{ TEXT("kmph"),		EUnit::KilometersPerHour },
	{ TEXT("MilesPerHour"),			EUnit::MilesPerHour },			{ TEXT("mi/h"),		EUnit::MilesPerHour },			{ TEXT("mph"),		EUnit::MilesPerHour },

	{ TEXT("DegreesPerSecond"),		EUnit::DegreesPerSecond },		{ TEXT("deg/s"),	EUnit::DegreesPerSecond },
	{ TEXT("RadiansPerSecond"),		EUnit::RadiansPerSecond },		{ TEXT("rad/s"),	EUnit::RadiansPerSecond },

	{ TEXT("CentimetersPerSecondSquared"),		EUnit::CentimetersPerSecondSquared },	{ TEXT("cm/s2"),	EUnit::CentimetersPerSecondSquared },{ TEXT("cm/s\u00B2"),	EUnit::CentimetersPerSecondSquared },{ TEXT("cm/s^2"),	EUnit::CentimetersPerSecondSquared },
	{ TEXT("MetersPerSecondSquared"),			EUnit::MetersPerSecondSquared },		{ TEXT("m/s2"),	EUnit::MetersPerSecondSquared },	{ TEXT("m/s\u00B2"),	EUnit::MetersPerSecondSquared },{ TEXT("m/s^2"),	EUnit::MetersPerSecondSquared },
		
	{ TEXT("Celsius"),				EUnit::Celsius },				{ TEXT("C"),		EUnit::Celsius },				{ TEXT("degC"),		EUnit::Celsius },			{ TEXT("\u00B0C"),		EUnit::Celsius },
	{ TEXT("Farenheit"),			EUnit::Farenheit },				{ TEXT("F"),		EUnit::Farenheit },				{ TEXT("degF"),		EUnit::Farenheit },			{ TEXT("\u00B0F"),		EUnit::Farenheit },
	{ TEXT("Kelvin"),				EUnit::Kelvin },				{ TEXT("K"),		EUnit::Kelvin },
					
	{ TEXT("Micrograms"),			EUnit::Micrograms },			{ TEXT("ug"),		EUnit::Micrograms }, 			{ TEXT("\u00B5g"),		EUnit::Micrograms },
	{ TEXT("Milligrams"),			EUnit::Milligrams },			{ TEXT("mg"),		EUnit::Milligrams },
	{ TEXT("Grams"),				EUnit::Grams },					{ TEXT("g"),		EUnit::Grams },
	{ TEXT("Kilograms"),			EUnit::Kilograms },				{ TEXT("kg"),		EUnit::Kilograms },
	{ TEXT("MetricTons"),			EUnit::MetricTons },			{ TEXT("t"),		EUnit::MetricTons },
	{ TEXT("Ounces"),				EUnit::Ounces },				{ TEXT("oz"),		EUnit::Ounces },
	{ TEXT("Pounds"),				EUnit::Pounds },				{ TEXT("lb"),		EUnit::Pounds },
	{ TEXT("Stones"),				EUnit::Stones },				{ TEXT("st"),		EUnit::Stones },

	{ TEXT("GramsPerCubicCentimeter"),		EUnit::GramsPerCubicCentimeter },		{ TEXT("g/cm3"),	EUnit::GramsPerCubicCentimeter },		{ TEXT("g/cm\u00B3"), EUnit::GramsPerCubicCentimeter },
	{ TEXT("GramsPerCubicMeter"),			EUnit::GramsPerCubicMeter },			{ TEXT("g/m3"),		EUnit::GramsPerCubicMeter },			{ TEXT("g/m\u00B3"), EUnit::GramsPerCubicMeter },
	{ TEXT("KilogramsPerCubicCentimeter"),	EUnit::KilogramsPerCubicCentimeter },	{ TEXT("kg/cm3"),	EUnit::KilogramsPerCubicCentimeter },	{ TEXT("kg/cm\u00B3"), EUnit::KilogramsPerCubicCentimeter },
	{ TEXT("KilogramsPerCubicMeter"),		EUnit::KilogramsPerCubicMeter },		{ TEXT("kg/m3"),	EUnit::KilogramsPerCubicMeter },		{ TEXT("kg/m\u00B3"), EUnit::KilogramsPerCubicMeter },

	{ TEXT("Newtons"),				EUnit::Newtons },				{ TEXT("N"),		EUnit::Newtons },
	{ TEXT("PoundsForce"),			EUnit::PoundsForce },			{ TEXT("lbf"),		EUnit::PoundsForce },
	{ TEXT("KilogramsForce"),		EUnit::KilogramsForce },		{ TEXT("kgf"),		EUnit::KilogramsForce },
	{ TEXT("KilogramsCentimetersPerSecondSquared"),	EUnit::KilogramCentimetersPerSecondSquared },	{ TEXT("kgcm/s2"),	EUnit::KilogramCentimetersPerSecondSquared },	{ TEXT("kgcm/s\u00B2"),	EUnit::KilogramCentimetersPerSecondSquared },

	{ TEXT("NewtonMeters"),			EUnit::NewtonMeters },			{ TEXT("Nm"),		EUnit::NewtonMeters },
	{ TEXT("KilogramsCentimetersSquaredPerSecondSquared"),	EUnit::KilogramCentimetersSquaredPerSecondSquared },		{ TEXT("kgcm2/s2"),	EUnit::KilogramCentimetersSquaredPerSecondSquared },	{ TEXT("kgcm\u00B2/s\u00B2"),	EUnit::KilogramCentimetersSquaredPerSecondSquared },

	{ TEXT("NewtonSeconds"),		EUnit::NewtonSeconds },			{ TEXT("Ns"),		EUnit::NewtonSeconds },
	{ TEXT("KilogramCentimeters"),	EUnit::KilogramCentimeters },	{ TEXT("kgcm"),		EUnit::KilogramCentimeters },
	{ TEXT("KilogramMeters"),		EUnit::KilogramMeters },		{ TEXT("kgm"),		EUnit::KilogramMeters },

	{TEXT("Decibels"),			EUnit::Decibels},					{TEXT("dB"),		EUnit::Decibels},
	
	{ TEXT("Hertz"),				EUnit::Hertz },					{ TEXT("Hz"),		EUnit::Hertz },
	{ TEXT("Kilohertz"),			EUnit::Kilohertz },				{ TEXT("KHz"),		EUnit::Kilohertz },
	{ TEXT("Megahertz"),			EUnit::Megahertz },				{ TEXT("MHz"),		EUnit::Megahertz },
	{ TEXT("Gigahertz"),			EUnit::Gigahertz },				{ TEXT("GHz"),		EUnit::Gigahertz },
	{ TEXT("RevolutionsPerMinute"),	EUnit::RevolutionsPerMinute },	{ TEXT("rpm"),		EUnit::RevolutionsPerMinute },

	{ TEXT("Bytes"),				EUnit::Bytes },					{ TEXT("B"),		EUnit::Bytes },
	{ TEXT("Kilobytes"),			EUnit::Kilobytes },				{ TEXT("KB"),		EUnit::Kilobytes },				{ TEXT("KiB"),		EUnit::Kilobytes },
	{ TEXT("Megabytes"),			EUnit::Megabytes },				{ TEXT("MB"),		EUnit::Megabytes },				{ TEXT("MiB"),		EUnit::Megabytes },
	{ TEXT("Gigabytes"),			EUnit::Gigabytes },				{ TEXT("GB"),		EUnit::Gigabytes },				{ TEXT("GiB"),		EUnit::Gigabytes },
	{ TEXT("Terabytes"),			EUnit::Terabytes },				{ TEXT("TB"),		EUnit::Terabytes },				{ TEXT("TiB"),		EUnit::Terabytes },

	{ TEXT("Lumens"),				EUnit::Lumens },				{ TEXT("lm"),		EUnit::Lumens },
	{ TEXT("Candela"),				EUnit::Candela },				{ TEXT("cd"),		EUnit::Candela },
	{ TEXT("Lux"),					EUnit::Lux },					{ TEXT("lx"),		EUnit::Lux },
	{ TEXT("CandelaPerMeterSquared"), EUnit::CandelaPerMeter2 },	{ TEXT("cd/m2"),	EUnit::CandelaPerMeter2 },		{ TEXT("CandelaPerMeter2"),		EUnit::CandelaPerMeter2 },
	{ TEXT("Nits"),                 EUnit::Nit },                   { TEXT("nt"),       EUnit::Nit },
	{ TEXT("EV"),					EUnit::ExposureValue },			{ TEXT("EV"),		EUnit::ExposureValue },

	{ TEXT("Nanoseconds"),			EUnit::Nanoseconds },			{ TEXT("ns"),		EUnit::Nanoseconds },
	{ TEXT("Microseconds"),			EUnit::Microseconds },			{ TEXT("us"),		EUnit::Microseconds },			{ TEXT("Microseconds"),			EUnit::Microseconds },			{ TEXT("\u00B5s"),	EUnit::Microseconds },
	{ TEXT("Milliseconds"),			EUnit::Milliseconds },			{ TEXT("ms"),		EUnit::Milliseconds },
	{ TEXT("Seconds"),				EUnit::Seconds },				{ TEXT("s"),		EUnit::Seconds },
	{ TEXT("Minutes"),				EUnit::Minutes },				{ TEXT("min"),		EUnit::Minutes },
	{ TEXT("Hours"),				EUnit::Hours },					{ TEXT("hrs"),		EUnit::Hours },
	{ TEXT("Days"),					EUnit::Days },					{ TEXT("dy"),		EUnit::Days },
	{ TEXT("Months"),				EUnit::Months },				{ TEXT("mth"),		EUnit::Months },
	{ TEXT("Years"),				EUnit::Years },					{ TEXT("yr"),		EUnit::Years },

	{ TEXT("ppi"),					EUnit::PixelsPerInch },			{ TEXT("dpi"),		EUnit::PixelsPerInch },

	{ TEXT("Percent"),				EUnit::Percentage },			{ TEXT("%"),	EUnit::Percentage },

	{ TEXT("Times"),				EUnit::Multiplier },			{ TEXT("x"),	EUnit::Multiplier },			{ TEXT("Multiplier"),		EUnit::Multiplier },

	{ TEXT("Pascals"),				EUnit::Pascals },				{ TEXT("Pa"),	EUnit::Pascals},
	{ TEXT("KiloPascals"),			EUnit::KiloPascals},			{ TEXT("kPa"),	EUnit::KiloPascals},
	{ TEXT("MegaPascals"),			EUnit::MegaPascals},			{ TEXT("MPa"),	EUnit::MegaPascals},
	{ TEXT("GigaPascals"),			EUnit::GigaPascals},			{ TEXT("GPa"),	EUnit::GigaPascals},
};

/** Static array of display strings that directly map to EUnit enumerations */
constexpr const TCHAR* const DisplayStrings[] = {
	TEXT("\u00B5m"),			TEXT("mm"),					TEXT("cm"),					TEXT("m"),					TEXT("km"),
	TEXT("in"),					TEXT("ft"),					TEXT("yd"),					TEXT("mi"),
	TEXT("ly"),

	TEXT("\u00B0"), TEXT("rad"),

	TEXT("cm/s"), TEXT("m/s"), TEXT("km/h"), TEXT("mi/h"),

	TEXT("deg/s"), TEXT("rad/s"),

	TEXT("cm/s\u00B2"), TEXT("m/s\u00B2"),

	TEXT("\u00B0C"), TEXT("\u00B0F"), TEXT("K"),

	TEXT("\u00B5g"), TEXT("mg"), TEXT("g"), TEXT("kg"), TEXT("t"),
	TEXT("oz"), TEXT("lb"), TEXT("st"),

	TEXT("g/cm\u00B3"), TEXT("g/m\u00B3"), TEXT("kg/cm\u00B3"), TEXT("kg/m\u00B3"),

	TEXT("N"), TEXT("lbf"), TEXT("kgf"), TEXT("kgcm/s\u00B2"),

	TEXT("Nm"), TEXT("kgcm\u00B2/s\u00B2"),

	TEXT("Ns"), TEXT("kgcm"), TEXT("kgm"),

	TEXT("dB"),
	
	TEXT("Hz"), TEXT("KHz"), TEXT("MHz"), TEXT("GHz"), TEXT("rpm"),

	TEXT("B"), TEXT("KiB"), TEXT("MiB"), TEXT("GiB"), TEXT("TiB"),

	TEXT("lm"), TEXT("cd"), TEXT("lux"), TEXT("cd/m2"), TEXT("nt"), TEXT("EV"),

	TEXT("ns"), TEXT("\u00B5s"), TEXT("ms"), TEXT("s"), TEXT("min"), TEXT("hr"), TEXT("dy"), TEXT("mth"), TEXT("yr"),

	TEXT("ppi"),

	TEXT("%"),

	TEXT("x"),

	TEXT("Pa"), TEXT("kPa"), TEXT("MPa"), TEXT("GPa"),
};

constexpr const TCHAR* const SupportedUnitsStrings[] = {
	TEXT("Micrometers"),
	TEXT("Millimeters"),		
	TEXT("Centimeters"),		
	TEXT("Meters"),				
	TEXT("Kilometers"),			
	TEXT("Inches"),				
	TEXT("Feet"),				
	TEXT("Yards"),				
	TEXT("Miles"),				
	TEXT("Lightyears"),			

	TEXT("Degrees"),				
	TEXT("Radians"),				
		
	TEXT("CentimetersPerSecond"),	
	TEXT("MetersPerSecond"),		
	TEXT("KilometersPerHour"),	
	TEXT("MilesPerHour"),			

	TEXT("DegreesPerSecond"),		
	TEXT("RadiansPerSecond"),
	
	TEXT("CentimetersPerSecondSquared"),		
	TEXT("MetersPerSecondSquared"),	
		
	TEXT("Celsius"),			
	TEXT("Farenheit"),			
	TEXT("Kelvin"),				
					
	TEXT("Micrograms"),			
	TEXT("Milligrams"),			
	TEXT("Grams"),				
	TEXT("Kilograms"),			
	TEXT("MetricTons"),			
	TEXT("Ounces"),				
	TEXT("Pounds"),				
	TEXT("Stones"),				

	TEXT("GramsPerCubicCentimeter"),	
	TEXT("GramsPerCubicMeter"),			
	TEXT("KilogramsPerCubicCentimeter"),
	TEXT("KilogramsPerCubicMeter"),		

	TEXT("Newtons"),			
	TEXT("PoundsForce"),		
	TEXT("KilogramsForce"),		
	TEXT("KilogramsCentimetersPerSecondSquared"),	

	TEXT("NewtonMeters"),			
	TEXT("KilogramsCentimetersSquaredPerSecondSquared"),

	TEXT("NewtonSeconds"),
	TEXT("KilogramCentimeters"),
	TEXT("KilogramMeters"),
	
	TEXT("Decibels"),

	TEXT("Hertz"),				
	TEXT("Kilohertz"),			
	TEXT("Megahertz"),			
	TEXT("Gigahertz"),			
	TEXT("RevolutionsPerMinute"),

	TEXT("Bytes"),				
	TEXT("Kilobytes"),		
	TEXT("Megabytes"),		
	TEXT("Gigabytes"),		
	TEXT("Terabytes"),		

	TEXT("Lumens"),				
	TEXT("Candela"),			
	TEXT("Lux"),				
	TEXT("CandelaPerMeterSquared"), 
	TEXT("Nits"), 
	TEXT("EV"),				

	TEXT("Nanoseconds"),			
	TEXT("Microseconds"),			
	TEXT("Milliseconds"),			
	TEXT("Seconds"),				
	TEXT("Minutes"),		
	TEXT("Hours"),			
	TEXT("Days"),			
	TEXT("Months"),			
	TEXT("Years"),			

	TEXT("ppi"),			

	TEXT("Percent"),

	TEXT("Times"),

	TEXT("Pascals"),		
	TEXT("KiloPascals"),	
	TEXT("MegaPascals"),	
	TEXT("GigaPascals")
};

static_assert(UE_ARRAY_COUNT(DisplayStrings) == UE_ARRAY_COUNT(SupportedUnitsStrings));
static_assert(UE_ARRAY_COUNT(DisplayStrings) == (uint32)EUnit::Unspecified);

constexpr EUnitType UnitTypes[] = {
	EUnitType::Distance,	EUnitType::Distance,	EUnitType::Distance,	EUnitType::Distance,	EUnitType::Distance,
	EUnitType::Distance,	EUnitType::Distance,	EUnitType::Distance,	EUnitType::Distance,
	EUnitType::Distance,

	EUnitType::Angle,		EUnitType::Angle,

	EUnitType::Speed,		EUnitType::Speed,		EUnitType::Speed, 		EUnitType::Speed,

	EUnitType::AngularSpeed, EUnitType::AngularSpeed,
	
	EUnitType::Acceleration, EUnitType::Acceleration,

	EUnitType::Temperature,	EUnitType::Temperature,	EUnitType::Temperature,

	EUnitType::Mass,		EUnitType::Mass,		EUnitType::Mass,		EUnitType::Mass,		EUnitType::Mass,
	EUnitType::Mass,		EUnitType::Mass,		EUnitType::Mass,

	EUnitType::Density,		EUnitType::Density,		EUnitType::Density,		EUnitType::Density,

	EUnitType::Force,		EUnitType::Force,		EUnitType::Force,		EUnitType::Force,

	EUnitType::Torque,		EUnitType::Torque,

	EUnitType::Impulse,		EUnitType::PositionalImpulse, EUnitType::PositionalImpulse,

	EUnitType::Loudness,
	
	EUnitType::Frequency,	EUnitType::Frequency,	EUnitType::Frequency,	EUnitType::Frequency,	EUnitType::Frequency,

	EUnitType::DataSize,	EUnitType::DataSize,	EUnitType::DataSize,	EUnitType::DataSize,	EUnitType::DataSize,

	EUnitType::LuminousFlux, EUnitType::LuminousIntensity, EUnitType::Illuminance, EUnitType::Luminance, EUnitType::Luminance, EUnitType::ExposureValue,

	EUnitType::Time,		EUnitType::Time,		EUnitType::Time,		EUnitType::Time,		EUnitType::Time,		EUnitType::Time,		EUnitType::Time,		EUnitType::Time,		EUnitType::Time,

	EUnitType::PixelDensity,

	EUnitType::Multipliers, EUnitType::Multipliers,

	EUnitType::Stress, EUnitType::Stress, EUnitType::Stress, EUnitType::Stress,
};

DEFINE_EXPRESSION_NODE_TYPE(FNumericUnit<double>, 0x3C138BC9, 0x71314F0B, 0xBB469BF7, 0xED47D147)

struct FUnitExpressionParser
{
	FUnitExpressionParser(EUnit InDefaultUnit)
	{
		using namespace ExpressionParser;

		TokenDefinitions.IgnoreWhitespace();
			
		// Defined in order of importance
		TokenDefinitions.DefineToken(&ConsumeSymbol<FPlusEquals>);
		TokenDefinitions.DefineToken(&ConsumeSymbol<FMinusEquals>);
		TokenDefinitions.DefineToken(&ConsumeSymbol<FStarEquals>);
		TokenDefinitions.DefineToken(&ConsumeSymbol<FForwardSlashEquals>);
		TokenDefinitions.DefineToken(&ConsumeSymbol<FPlus>);
		TokenDefinitions.DefineToken(&ConsumeSymbol<FMinus>);
		TokenDefinitions.DefineToken(&ConsumeSymbol<FStar>);
		TokenDefinitions.DefineToken(&ConsumeSymbol<FForwardSlash>);
		TokenDefinitions.DefineToken(&ConsumeSymbol<FSubExpressionStart>);
		TokenDefinitions.DefineToken(&ConsumeSymbol<FSubExpressionEnd>);

		TokenDefinitions.DefineToken([&](FExpressionTokenConsumer& Consumer){ return ConsumeNumberWithUnits(Consumer); });

		Grammar.DefineGrouping<FSubExpressionStart, FSubExpressionEnd>();
		
		Grammar.DefinePreUnaryOperator<FPlus>();
		Grammar.DefinePreUnaryOperator<FMinus>();

		Grammar.DefineBinaryOperator<FPlus>(5);
		Grammar.DefineBinaryOperator<FMinus>(5);
		Grammar.DefineBinaryOperator<FStar>(4);
		Grammar.DefineBinaryOperator<FForwardSlash>(4);

		// Unary operators for numeric units
		JumpTable.MapPreUnary<FPlus>(	[](const FNumericUnit<double>& N) {	return FNumericUnit<double>(N.Value, N.Units);		});
		JumpTable.MapPreUnary<FMinus>(	[](const FNumericUnit<double>& N) {	return FNumericUnit<double>(-N.Value, N.Units);		});

		/** Addition for numeric units */
		JumpTable.MapBinary<FPlus>([InDefaultUnit](const FNumericUnit<double>& A, const FNumericUnit<double>& B) -> FExpressionResult {

			// Have to ensure we're adding the correct units here

			EUnit UnitsLHS = A.Units, UnitsRHS = B.Units;

			if (UnitsLHS == EUnit::Unspecified && UnitsRHS != EUnit::Unspecified)
			{
				UnitsLHS = InDefaultUnit;
			}
			else if (UnitsLHS != EUnit::Unspecified && UnitsRHS == EUnit::Unspecified)
			{
				UnitsRHS = InDefaultUnit;
			}

			if (FUnitConversion::AreUnitsCompatible(UnitsLHS, UnitsRHS))
			{
				if (UnitsLHS != EUnit::Unspecified)
				{
					return MakeValue(FNumericUnit<double>(A.Value + FUnitConversion::Convert(B.Value, UnitsRHS, UnitsLHS), UnitsLHS));
				}
				else
				{
					return MakeValue(FNumericUnit<double>(FUnitConversion::Convert(A.Value, UnitsLHS, UnitsRHS) + B.Value, UnitsRHS));
				}
			}

			FFormatOrderedArguments Args;
			Args.Add(FText::FromString(FUnitConversion::GetUnitDisplayString(B.Units)));
			Args.Add(FText::FromString(FUnitConversion::GetUnitDisplayString(A.Units)));
			return MakeError(FText::Format(LOCTEXT("CannotAddErr", "Cannot add {0} to {1}"), Args));
		});

		/** Subtraction for numeric units */
		JumpTable.MapBinary<FMinus>([InDefaultUnit](const FNumericUnit<double>& A, const FNumericUnit<double>& B) -> FExpressionResult {
			
			// Have to ensure we're adding the correct units here

			EUnit UnitsLHS = A.Units, UnitsRHS = B.Units;

			if (UnitsLHS == EUnit::Unspecified && UnitsRHS != EUnit::Unspecified)
			{
				UnitsLHS = InDefaultUnit;
			}
			else if (UnitsLHS != EUnit::Unspecified && UnitsRHS == EUnit::Unspecified)
			{
				UnitsRHS = InDefaultUnit;
			}

			if (FUnitConversion::AreUnitsCompatible(UnitsLHS, UnitsRHS))
			{
				if (UnitsLHS != EUnit::Unspecified)
				{
					return MakeValue(FNumericUnit<double>(A.Value - FUnitConversion::Convert(B.Value, UnitsRHS, UnitsLHS), UnitsLHS));
				}
				else
				{
					return MakeValue(FNumericUnit<double>(FUnitConversion::Convert(A.Value, UnitsLHS, UnitsRHS) - B.Value, UnitsRHS));
				}
			}

			FFormatOrderedArguments Args;
			Args.Add(FText::FromString(FUnitConversion::GetUnitDisplayString(B.Units)));
			Args.Add(FText::FromString(FUnitConversion::GetUnitDisplayString(A.Units)));
			return MakeError(FText::Format(LOCTEXT("CannotSubtractErr", "Cannot subtract {1} from {0}"), Args));
		});

		/** Multiplication */
		JumpTable.MapBinary<FStar>([](const FNumericUnit<double>& A, const FNumericUnit<double>& B) -> FExpressionResult {
			if (A.Units != EUnit::Unspecified && B.Units != EUnit::Unspecified)
			{
				return MakeError(LOCTEXT("InvalidMultiplication", "Cannot multiply by numbers with units"));
			}
			return MakeValue(FNumericUnit<double>(B.Value*A.Value, A.Units == EUnit::Unspecified ? B.Units : A.Units));
		});

		/** Division */
		JumpTable.MapBinary<FForwardSlash>([](const FNumericUnit<double>& A, const FNumericUnit<double>& B) -> FExpressionResult {
			if (B.Units != EUnit::Unspecified)
			{
				return MakeError(LOCTEXT("InvalidDivision", "Cannot divide by numbers with units"));
			}
			else if (B.Value == 0)
			{
				return MakeError(LOCTEXT("DivideByZero", "DivideByZero"));	
			}
			return MakeValue(FNumericUnit<double>(A.Value/B.Value, A.Units));
		});
	}

	/** Consume a number from the stream, optionally including units */
	TOptional<FExpressionError> ConsumeNumberWithUnits(FExpressionTokenConsumer& Consumer)
	{
		auto& Stream = Consumer.GetStream();

		double Value = 0.0;
		TOptional<FStringToken> NumberToken = ExpressionParser::ParseLocalizedNumberWithAgnosticFallback(Stream, nullptr, &Value);
		
		if (NumberToken.IsSet())
		{
			// Skip past any additional whitespace between number and unit token, e.g "1.0 cm"
			const TOptional<FStringToken> WhiteSpaceToken = Stream.ParseWhitespace(&NumberToken.GetValue());

			size_t LongestMatch = 0;
			TOptional<EUnit> Unit;
			FStringToken UnitToken;
			const FStringToken StartingToken = NumberToken.GetValue();

			for (int32 Index = 0; Index < UE_ARRAY_COUNT(ParseCandidates); ++Index)
			{
				TOptional<FStringToken> CandidateUnitToken = Stream.ParseTokenIgnoreCase(ParseCandidates[Index].String, &NumberToken.GetValue());
				const size_t MatchedCharCount = NumberToken.GetValue().GetTokenEndPos() - StartingToken.GetTokenEndPos();
				if (CandidateUnitToken.IsSet() && MatchedCharCount > LongestMatch)
				{
					Unit = ParseCandidates[Index].Unit;
					UnitToken = CandidateUnitToken.GetValue();
					LongestMatch = MatchedCharCount;
				}
				else
				{
					NumberToken = StartingToken;
				}
			}

			if (Unit.IsSet())
			{
				Consumer.Add(UnitToken, FNumericUnit<double>(Value, Unit.GetValue()));
			}
			else
			{
				Consumer.Add(NumberToken.GetValue(), FNumericUnit<double>(Value));
			}
		}

		return TOptional<FExpressionError>();
	}

	TValueOrError<FNumericUnit<double>, FExpressionError> Evaluate(const TCHAR* InExpression, const FNumericUnit<double>& InExistingValue) const
	{
		using namespace ExpressionParser;

		TValueOrError<TArray<FExpressionToken>, FExpressionError> LexResult = ExpressionParser::Lex(InExpression, TokenDefinitions);
		if (!LexResult.IsValid())
		{
			return MakeError(LexResult.StealError());
		}

		// Handle the += and -= tokens.
		TArray<FExpressionToken> Tokens = LexResult.StealValue();
		if (Tokens.Num())
		{
			FStringToken Context = Tokens[0].Context;
			const FExpressionNode& FirstNode = Tokens[0].Node;
			bool WasOpAssign = true;

			if (FirstNode.Cast<FPlusEquals>())
			{			
				Tokens.Insert(FExpressionToken(Context, FPlus()), 0);
			}
			else if (FirstNode.Cast<FMinusEquals>())
			{
				Tokens.Insert(FExpressionToken(Context, FMinus()), 0);
			}
			else if (FirstNode.Cast<FStarEquals>())
			{
				Tokens.Insert(FExpressionToken(Context, FStar()), 0);
			}
			else if (FirstNode.Cast<FForwardSlashEquals>())
			{
				Tokens.Insert(FExpressionToken(Context, FForwardSlash()), 0);
			}
			else
			{
				WasOpAssign = false;
			}

			if (WasOpAssign)
			{
				Tokens.Insert(FExpressionToken(Context, InExistingValue), 0);
				Tokens.RemoveAt(2, EAllowShrinking::No);
			}
		}

		TValueOrError<TArray<FCompiledToken>, FExpressionError> CompilationResult = ExpressionParser::Compile(MoveTemp(Tokens), Grammar);
		if (!CompilationResult.IsValid())
		{
			return MakeError(CompilationResult.StealError());
		}

		TOperatorEvaluationEnvironment<> Env(JumpTable, nullptr);
		TValueOrError<FExpressionNode, FExpressionError> Result = ExpressionParser::Evaluate(CompilationResult.GetValue(), Env);
		if (!Result.IsValid())
		{
			return MakeError(Result.GetError());
		}

		auto& Node = Result.GetValue();
		if (const auto* Numeric = Node.Cast<double>())
		{
			return MakeValue(FNumericUnit<double>(*Numeric, EUnit::Unspecified));
		}
		else if (const auto* NumericUnit = Node.Cast<FNumericUnit<double>>())
		{
			return MakeValue(*NumericUnit);
		}
		else
		{
			return MakeError(LOCTEXT("UnrecognizedResult", "Unrecognized result returned from expression"));
		}
	}
	
private:
	FTokenDefinitions TokenDefinitions;
	FExpressionGrammar Grammar;
	FOperatorJumpTable JumpTable;
};

FUnitSettings::FUnitSettings()
	: bGlobalUnitDisplay(true)
{
	DisplayUnits[(uint8)EUnitType::Distance].Add(EUnit::Centimeters);
	DisplayUnits[(uint8)EUnitType::Angle].Add(EUnit::Degrees);
	DisplayUnits[(uint8)EUnitType::Speed].Add(EUnit::MetersPerSecond);
	DisplayUnits[(uint8)EUnitType::AngularSpeed].Add(EUnit::DegreesPerSecond);
	DisplayUnits[(uint8)EUnitType::Acceleration].Add(EUnit::CentimetersPerSecondSquared);
	DisplayUnits[(uint8)EUnitType::Temperature].Add(EUnit::Celsius);
	DisplayUnits[(uint8)EUnitType::Mass].Add(EUnit::Kilograms);
	DisplayUnits[(uint8)EUnitType::Density].Add(EUnit::GramsPerCubicCentimeter);
	DisplayUnits[(uint8)EUnitType::Force].Add(EUnit::Newtons);
	DisplayUnits[(uint8)EUnitType::Torque].Add(EUnit::NewtonMeters);
	DisplayUnits[(uint8)EUnitType::Impulse].Add(EUnit::NewtonSeconds);
	DisplayUnits[(uint8)EUnitType::PositionalImpulse].Add(EUnit::KilogramCentimeters);
	DisplayUnits[(uint8)EUnitType::Frequency].Add(EUnit::Hertz);
	DisplayUnits[(uint8)EUnitType::DataSize].Add(EUnit::Megabytes);
	DisplayUnits[(uint8)EUnitType::LuminousFlux].Add(EUnit::Lumens);
	DisplayUnits[(uint8)EUnitType::ExposureValue].Add(EUnit::ExposureValue);
	DisplayUnits[(uint8)EUnitType::Time].Add(EUnit::Seconds);
	DisplayUnits[(uint8)EUnitType::Stress].Add(EUnit::MegaPascals);
}

bool FUnitSettings::ShouldDisplayUnits() const
{
	return bGlobalUnitDisplay;
}

void FUnitSettings::SetShouldDisplayUnits(bool bInGlobalUnitDisplay)
{
	bGlobalUnitDisplay = bInGlobalUnitDisplay;
	SettingChangedEvent.Broadcast();
}

const TArray<EUnit>& FUnitSettings::GetDisplayUnits(EUnitType InType) const
{
	return DisplayUnits[(uint8)InType];
}

void FUnitSettings::SetDisplayUnits(EUnitType InType, EUnit Unit)
{
	if (InType != EUnitType::NumberOf)
	{
		DisplayUnits[(uint8)InType].Empty();
		if (FUnitConversion::IsUnitOfType(Unit, InType))
		{
			DisplayUnits[(uint8)InType].Add(Unit);
		}

		SettingChangedEvent.Broadcast();
	}	
}

void FUnitSettings::SetDisplayUnits(EUnitType InType, const TArray<EUnit>& Units)
{
	if (InType != EUnitType::NumberOf)
	{
		DisplayUnits[(uint8)InType].Reset();
		for (EUnit Unit : Units)
		{
			if (FUnitConversion::IsUnitOfType(Unit, InType))
			{
				DisplayUnits[(uint8)InType].Add(Unit);
			}
		}

		SettingChangedEvent.Broadcast();
	}
}

/** Get the global settings for unit conversion/display */
FUnitSettings& FUnitConversion::Settings()
{
	static TUniquePtr<FUnitSettings> Settings(new FUnitSettings);
	return *Settings;
}

/** Check whether it is possible to convert a number between the two specified units */
bool FUnitConversion::AreUnitsCompatible(EUnit From, EUnit To)
{
	return From == EUnit::Unspecified || To == EUnit::Unspecified || GetUnitType(From) == GetUnitType(To);
}

/** Check whether a unit is of the specified type */
bool FUnitConversion::IsUnitOfType(EUnit Unit, EUnitType Type)
{
	return Unit != EUnit::Unspecified && GetUnitType(Unit) == Type;
}

/** Get the type of the specified unit */
EUnitType FUnitConversion::GetUnitType(EUnit InUnit)
{
	if (ensure(InUnit != EUnit::Unspecified))
	{
		return UnitTypes[(uint8)InUnit];
	}
	return EUnitType::NumberOf;
}

/** Get the unit abbreviation the specified unit type */
const TCHAR* FUnitConversion::GetUnitDisplayString(EUnit Unit)
{
	static_assert(UE_ARRAY_COUNT(UnitTypes) == (uint8)EUnit::Unspecified, "Type array does not match size of unit enum");
	static_assert(UE_ARRAY_COUNT(DisplayStrings) == (uint8)EUnit::Unspecified, "Display String array does not match size of unit enum");
	
	if (Unit != EUnit::Unspecified)
	{
		return DisplayStrings[(uint8)Unit];
	}
	return nullptr;
}

/** Helper function to find a unit from a string (name or abbreviation) */
TOptional<EUnit> FUnitConversion::UnitFromString(const TCHAR* UnitString)
{
	if (!UnitString || *UnitString == '\0')
	{
		return TOptional<EUnit>();
	}

	for (int32 Index = 0; Index < UE_ARRAY_COUNT(ParseCandidates); ++Index)
	{
		if (FCString::Stricmp(UnitString, ParseCandidates[Index].String) == 0)
		{
			return ParseCandidates[Index].Unit;
		}
	}

	return TOptional<EUnit>();
}

TConstArrayView<const TCHAR*> FUnitConversion::GetSupportedUnits()
{
	return TConstArrayView<const TCHAR*>(SupportedUnitsStrings,  UE_ARRAY_COUNT(SupportedUnitsStrings));
}

namespace UnitConversion
{

	double DistanceUnificationFactor(EUnit From)
	{
		// Convert to meters
		double Factor = 1;
		switch (From)
		{
			case EUnit::Micrometers:		return 0.000001;
			case EUnit::Millimeters:		return 0.001;
			case EUnit::Centimeters:		return 0.01;
			case EUnit::Kilometers:			return 1000;

			case EUnit::Lightyears:			return 9.4605284e15;

			case EUnit::Miles:				Factor *= 1760;				// fallthrough
			case EUnit::Yards:				Factor *= 3;				// fallthrough
			case EUnit::Feet:				Factor *= 12;				// fallthrough
			case EUnit::Inches:				Factor /= 39.3700787;		// fallthrough
			default: 						return Factor;				// return
		}
	}

	double AngleUnificationFactor(EUnit From)
	{
		// Convert to degrees
		switch (From)
		{
			case EUnit::Radians:			return (180 / UE_PI);
			default: 						return 1;
		}
	}

	double SpeedUnificationFactor(EUnit From)
	{
		// Convert to km/h
		switch (From)
		{
			case EUnit::CentimetersPerSecond:	return 0.036;
			case EUnit::MetersPerSecond:		return 3.6;
			case EUnit::MilesPerHour:			return DistanceUnificationFactor(EUnit::Miles) / 1000;
			default: 							return 1;
		}
	}


	double AngularSpeedUnificationFactor(EUnit From)
	{
		// Convert to degrees/second
		switch (From)
		{
			case EUnit::RadiansPerSecond:	return (180.0 / UE_DOUBLE_PI);
			default: 						return 1.0;
		}
	}
	
	double AccelerationUnificationFactor(EUnit From)
	{
		// Convert to meters/second2
		switch (From)
		{
		case EUnit::CentimetersPerSecondSquared:	return 0.01f;
		default: 									return 1.0;
		}
	}

	double MassUnificationFactor(EUnit From)
	{
		double Factor = 1;
		// Convert to grams
		switch (From)
		{
			case EUnit::Micrograms:			return 0.000001;
			case EUnit::Milligrams:			return 0.001;
			case EUnit::Kilograms:			return 1000;
			case EUnit::MetricTons:			return 1000000;

			case EUnit::Stones:				Factor *= 14;		// fallthrough
			case EUnit::Pounds:				Factor *= 16;		// fallthrough
			case EUnit::Ounces:				Factor *= 28.3495;	// fallthrough
			default: 						return Factor;		// return
		}
	}

	double DensityUnificationFactor(EUnit From)
	{
		// Convert to g/cm^3
		switch (From)
		{
		case EUnit::GramsPerCubicCentimeter:		return 1;
		case EUnit::GramsPerCubicMeter:				return 0.000001;
		case EUnit::KilogramsPerCubicCentimeter:	return 1000;
		case EUnit::KilogramsPerCubicMeter:			return 0.001;
		default:									return 1;
		}
	}

	double ForceUnificationFactor(EUnit From)
	{
		// Convert to Newtons
		switch (From)
		{
		case EUnit::PoundsForce:							return 4.44822162;
		case EUnit::KilogramsForce:							return 9.80665;
		case EUnit::KilogramCentimetersPerSecondSquared:	return 0.01;
		default: 											return 1;
		}
	}

	double TorqueUnificationFactor(EUnit From)
	{
		// Convert to NewtonMeters
		switch (From)
		{
		case EUnit::KilogramCentimetersSquaredPerSecondSquared:	return 0.0001;
		default: 												return 1;
		}
	}

	double PositionalImpulseUnificationFactor(EUnit From)
	{
		// Convert to KilogramMeters
		switch (From)
		{
		case EUnit::KilogramCentimeters:	return 0.01;
		default:							return 1;
		}
	}

	double FrequencyUnificationFactor(EUnit From)
	{
		// Convert to KHz
		switch (From)
		{
			case EUnit::Hertz:				return 0.001;
			case EUnit::Megahertz:			return 1000;
			case EUnit::Gigahertz:			return 1000000;

			case EUnit::RevolutionsPerMinute:	return 0.001/60;

			default: 						return 1;
		}
	}

	double DataSizeUnificationFactor(EUnit From)
	{
		// Convert to MB
		switch (From)
		{
			case EUnit::Bytes:				return 1.0/(1024*1024);
			case EUnit::Kilobytes:			return 1.0/1024;
			case EUnit::Gigabytes:			return 1024;
			case EUnit::Terabytes:			return 1024*1024;

			default: 						return 1;
		}
	}

	double TimeUnificationFactor(EUnit From)
	{
		// Convert to hours
		double Factor = 1;
		switch (From)
		{
			case EUnit::Months:				return (365.242 * 24) / 12;

			case EUnit::Years:				Factor *= 365.242;	// fallthrough
			case EUnit::Days:				Factor *= 24;		// fallthrough
											return Factor;

			case EUnit::Nanoseconds:		Factor /= 1000;		// fallthrough
			case EUnit::Microseconds:		Factor /= 1000;		// fallthrough
			case EUnit::Milliseconds:		Factor /= 1000;		// fallthrough
			case EUnit::Seconds:			Factor /= 60;		// fallthrough
			case EUnit::Minutes:			Factor /= 60;		// fallthrough
											return Factor;

			default: 						return 1;
		}
	}

	double MultiplierUnificationFactor(EUnit From)
	{
		switch (From)
		{
			case EUnit::Percentage:			return 0.01;
			case EUnit::Multiplier:						// fallthrough
			default: 						return 1.0;
		}
	}

	double StressUnificationFactor(EUnit From)
	{
		// Convert to Pascals
		switch (From)
		{
		case EUnit::GigaPascals:		return 1000000000.0;
		case EUnit::MegaPascals:		return 1000000.0;
		case EUnit::KiloPascals:		return 1000.0;
		case EUnit::Pascals:			// fallthrough
		default: 						return 1.0;
		}
	}

	TValueOrError<FNumericUnit<double>, FText> TryParseExpression(const TCHAR* InExpression, EUnit From, const FNumericUnit<double>& InExistingValue)
	{
		const FUnitExpressionParser Parser(From);
		auto Result = Parser.Evaluate(InExpression, InExistingValue);
		if (Result.IsValid())
		{
			if (Result.GetValue().Units == EUnit::Unspecified)
			{
				return MakeValue(FNumericUnit<double>(Result.GetValue().Value, From));
			}
			else
			{
				return MakeValue(Result.GetValue());
			}
		}
		else
		{
			return MakeError(Result.GetError().Text);
		}
	}

	TOptional<const TArray<FQuantizationInfo>*> GetQuantizationBounds(EUnit Unit)
	{
		struct FStaticBounds
		{
			TArray<FQuantizationInfo> MetricDistance;
			TArray<FQuantizationInfo> ImperialDistance;

			TArray<FQuantizationInfo> MetricMass;
			TArray<FQuantizationInfo> ImperialMass;

			TArray<FQuantizationInfo> MetricDensity;

			TArray<FQuantizationInfo> Frequency;
			TArray<FQuantizationInfo> DataSize;

			TArray<FQuantizationInfo> Time;

			TArray<FQuantizationInfo> Stress;

			FStaticBounds()
			{
				MetricDistance.Emplace(EUnit::Micrometers,	1000.0f);
				MetricDistance.Emplace(EUnit::Millimeters,	10.0f);
				MetricDistance.Emplace(EUnit::Centimeters,	100.0f);
				MetricDistance.Emplace(EUnit::Meters,			1000.0f);
				MetricDistance.Emplace(EUnit::Kilometers,		0.0f);

				ImperialDistance.Emplace(EUnit::Inches,	12.0f);
				ImperialDistance.Emplace(EUnit::Feet,		3.0f);
				ImperialDistance.Emplace(EUnit::Yards,	1760.0f);
				ImperialDistance.Emplace(EUnit::Miles,	0.0f);

				MetricMass.Emplace(EUnit::Micrograms,	1000.0f);
				MetricMass.Emplace(EUnit::Milligrams,	1000.0f);
				MetricMass.Emplace(EUnit::Grams,		1000.0f);
				MetricMass.Emplace(EUnit::Kilograms,	1000.0f);
				MetricMass.Emplace(EUnit::MetricTons,	0.0f);

				ImperialMass.Emplace(EUnit::Ounces,	16.0f);
				ImperialMass.Emplace(EUnit::Pounds,	14.0f);
				ImperialMass.Emplace(EUnit::Stones,	0.0f);

				MetricDensity.Emplace(EUnit::GramsPerCubicCentimeter,		0.000001f);
				MetricDensity.Emplace(EUnit::GramsPerCubicMeter,			1000.0f);
				MetricDensity.Emplace(EUnit::KilogramsPerCubicCentimeter,	0.000001f);
				MetricDensity.Emplace(EUnit::KilogramsPerCubicMeter,		0.0f);

				Frequency.Emplace(EUnit::Hertz,		1000.0f);
				Frequency.Emplace(EUnit::Kilohertz,	1000.0f);
				Frequency.Emplace(EUnit::Megahertz,	1000.0f);
				Frequency.Emplace(EUnit::Gigahertz,	0.0f);

				DataSize.Emplace(EUnit::Bytes,		1024.0f);
				DataSize.Emplace(EUnit::Kilobytes,	1024.0f);
				DataSize.Emplace(EUnit::Megabytes,	1024.0f);
				DataSize.Emplace(EUnit::Gigabytes,	1024.0f);
				DataSize.Emplace(EUnit::Terabytes,	0.0f);

				Time.Emplace(EUnit::Microseconds,	1000.0f);
				Time.Emplace(EUnit::Milliseconds,	1000.0f);
				Time.Emplace(EUnit::Seconds,		60.0f);
				Time.Emplace(EUnit::Minutes,		60.0f);
				Time.Emplace(EUnit::Hours,			24.0f);
				Time.Emplace(EUnit::Days,			365.242f / 12.0f);
				Time.Emplace(EUnit::Months,			12.0f);
				Time.Emplace(EUnit::Years,			0.0f);

				Stress.Emplace(EUnit::Pascals,		1000.0f);
				Stress.Emplace(EUnit::KiloPascals,	1000.0f);
				Stress.Emplace(EUnit::MegaPascals,	1000.0f);
				Stress.Emplace(EUnit::GigaPascals,	0.0f);
			}
		};

		static FStaticBounds Bounds;

		switch (Unit)
		{
		case EUnit::Micrometers: case EUnit::Millimeters: case EUnit::Centimeters: case EUnit::Meters: case EUnit::Kilometers:
			return &Bounds.MetricDistance;

		case EUnit::Inches: case EUnit::Feet: case EUnit::Yards: case EUnit::Miles:
			return &Bounds.ImperialDistance;

		case EUnit::Micrograms: case EUnit::Milligrams: case EUnit::Grams: case EUnit::Kilograms: case EUnit::MetricTons:
			return &Bounds.MetricMass;

		case EUnit::Ounces: case EUnit::Pounds: case EUnit::Stones:
			return &Bounds.ImperialMass;

		case EUnit::GramsPerCubicCentimeter:
		case EUnit::GramsPerCubicMeter:
		case EUnit::KilogramsPerCubicCentimeter:
		case EUnit::KilogramsPerCubicMeter:
			return &Bounds.MetricDensity;

		case EUnit::Hertz: case EUnit::Kilohertz: case EUnit::Megahertz: case EUnit::Gigahertz: case EUnit::RevolutionsPerMinute:
			return &Bounds.Frequency;

		case EUnit::Bytes: case EUnit::Kilobytes: case EUnit::Megabytes: case EUnit::Gigabytes: case EUnit::Terabytes:
			return &Bounds.DataSize;

		case EUnit::Microseconds: case EUnit::Milliseconds: case EUnit::Seconds: case EUnit::Minutes: case EUnit::Hours: case EUnit::Days: case EUnit::Months: case EUnit::Years:
			return &Bounds.Time;

		case EUnit::Pascals: case EUnit::KiloPascals: case EUnit::MegaPascals: case EUnit::GigaPascals:
			return &Bounds.Stress;

		default:
			return TOptional<const TArray<FQuantizationInfo>*>();
		}
	}

}	// namespace UnitConversion

#undef LOCTEXT_NAMESPACE

PRAGMA_RESTORE_UNSAFE_TYPECAST_WARNINGS
