// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	UnMath.cpp: Unreal math routines
=============================================================================*/

#include "Math/UnrealMath.h"
#include "Stats/Stats.h"
#include "Math/RandomStream.h"
#include "UObject/PropertyPortFlags.h"
#include "Math/IntRect.h"
#include "Math/Matrix.h"
#include "Math/Quat.h"
#include "Math/Vector.h"
#include "Math/Vector2D.h"
#include "Math/Vector4.h"

DEFINE_LOG_CATEGORY(LogUnrealMath);

/*-----------------------------------------------------------------------------
	Globals
-----------------------------------------------------------------------------*/

template<> const FMatrix44f FMatrix44f::Identity(FPlane4f(1, 0, 0, 0), FPlane4f(0, 1, 0, 0), FPlane4f(0, 0, 1, 0), FPlane4f(0, 0, 0, 1));
template<> const FMatrix44d FMatrix44d::Identity(FPlane4d(1, 0, 0, 0), FPlane4d(0, 1, 0, 0), FPlane4d(0, 0, 1, 0), FPlane4d(0, 0, 0, 1));

template<> const FQuat4f FQuat4f::Identity(0.f, 0.f, 0.f, 1.f);
template<> const FQuat4d FQuat4d::Identity(0.0, 0.0, 0.0, 1.0);

template<> const FRotator3f FRotator3f::ZeroRotator(0, 0, 0);
template<> const FRotator3d FRotator3d::ZeroRotator(0, 0, 0);

template<> const FVector3f FVector3f::ZeroVector(0, 0, 0);
template<> const FVector3f FVector3f::OneVector(1, 1, 1);
template<> const FVector3f FVector3f::UpVector(0, 0, 1);
template<> const FVector3f FVector3f::DownVector(0, 0, -1);
template<> const FVector3f FVector3f::ForwardVector(1, 0, 0);
template<> const FVector3f FVector3f::BackwardVector(-1, 0, 0);
template<> const FVector3f FVector3f::RightVector(0, 1, 0);
template<> const FVector3f FVector3f::LeftVector(0, -1, 0);
template<> const FVector3f FVector3f::XAxisVector(1, 0, 0);
template<> const FVector3f FVector3f::YAxisVector(0, 1, 0);
template<> const FVector3f FVector3f::ZAxisVector(0, 0, 1);
template<> const FVector3d FVector3d::ZeroVector(0, 0, 0);
template<> const FVector3d FVector3d::OneVector(1, 1, 1);
template<> const FVector3d FVector3d::UpVector(0, 0, 1);
template<> const FVector3d FVector3d::DownVector(0, 0, -1);
template<> const FVector3d FVector3d::ForwardVector(1, 0, 0);
template<> const FVector3d FVector3d::BackwardVector(-1, 0, 0);
template<> const FVector3d FVector3d::RightVector(0, 1, 0);
template<> const FVector3d FVector3d::LeftVector(0, -1, 0);
template<> const FVector3d FVector3d::XAxisVector(1, 0, 0);
template<> const FVector3d FVector3d::YAxisVector(0, 1, 0);
template<> const FVector3d FVector3d::ZAxisVector(0, 0, 1);

template<> const FVector2f FVector2f::ZeroVector(0, 0);
template<> const FVector2f FVector2f::UnitVector(1, 1);
template<> const FVector2f FVector2f::Unit45Deg(UE_INV_SQRT_2, UE_INV_SQRT_2);
template<> const FVector2d FVector2d::ZeroVector(0, 0);
template<> const FVector2d FVector2d::UnitVector(1, 1);
template<> const FVector2d FVector2d::Unit45Deg(UE_INV_SQRT_2, UE_INV_SQRT_2);

CORE_API const uint32 FMath::BitFlag[32] =
{
	(1U << 0),	(1U << 1),	(1U << 2),	(1U << 3),
	(1U << 4),	(1U << 5),	(1U << 6),	(1U << 7),
	(1U << 8),	(1U << 9),	(1U << 10),	(1U << 11),
	(1U << 12),	(1U << 13),	(1U << 14),	(1U << 15),
	(1U << 16),	(1U << 17),	(1U << 18),	(1U << 19),
	(1U << 20),	(1U << 21),	(1U << 22),	(1U << 23),
	(1U << 24),	(1U << 25),	(1U << 26),	(1U << 27),
	(1U << 28),	(1U << 29),	(1U << 30),	(1U << 31),
};

template<typename T>
bool UE::Math::TRotator<T>::NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess)
{
	SerializeCompressedShort( Ar );
	bOutSuccess = true;
	return true;
}

template<typename T>
void UE::Math::TRotator<T>::SerializeCompressed( FArchive& Ar )
{
	const bool bArLoading = Ar.IsLoading();
	
	uint8 BytePitch = 0;
	uint8 ByteYaw = 0;
	uint8 ByteRoll = 0;
	
	// If saving, we need to compress before writing. If loading we'll just serialize in the data so no need to compress.
	if( !bArLoading )
	{
		BytePitch = TRotator<T>::CompressAxisToByte(Pitch);
		ByteYaw = TRotator<T>::CompressAxisToByte(Yaw);
		ByteRoll = TRotator<T>::CompressAxisToByte(Roll);
	}

	uint8 B = (BytePitch!=0);
	Ar.SerializeBits( &B, 1 );
	if( B )
	{
		Ar << BytePitch;
	}
	else
	{
		BytePitch = 0;
	}
	
	B = (ByteYaw!=0);
	Ar.SerializeBits( &B, 1 );
	if( B )
	{
		Ar << ByteYaw;
	}
	else
	{
		ByteYaw = 0;
	}
	
	B = (ByteRoll!=0);
	Ar.SerializeBits( &B, 1 );
	if( B )
	{
		Ar << ByteRoll;
	}
	else
	{
		ByteRoll = 0;
	}
	
	if( bArLoading )
	{
		Pitch = TRotator<T>::DecompressAxisFromByte(BytePitch);
		Yaw	= TRotator<T>::DecompressAxisFromByte(ByteYaw);
		Roll = TRotator<T>::DecompressAxisFromByte(ByteRoll);
	}
}

template<typename T>
void UE::Math::TRotator<T>::SerializeCompressedShort( FArchive& Ar )
{
	const bool bArLoading = Ar.IsLoading();

	uint16 ShortPitch = 0;
	uint16 ShortYaw = 0;
	uint16 ShortRoll = 0;

	// If saving, we need to compress before writing. If loading we'll just serialize in the data so no need to compress.
	if( !bArLoading )
	{
		ShortPitch = TRotator<T>::CompressAxisToShort(Pitch);
		ShortYaw = TRotator<T>::CompressAxisToShort(Yaw);
		ShortRoll = TRotator<T>::CompressAxisToShort(Roll);
	}

	uint8 B = (ShortPitch!=0);
	Ar.SerializeBits( &B, 1 );
	if( B )
	{
		Ar << ShortPitch;
	}
	else
	{
		ShortPitch = 0;
	}

	B = (ShortYaw!=0);
	Ar.SerializeBits( &B, 1 );
	if( B )
	{
		Ar << ShortYaw;
	}
	else
	{
		ShortYaw = 0;
	}

	B = (ShortRoll!=0);
	Ar.SerializeBits( &B, 1 );
	if( B )
	{
		Ar << ShortRoll;
	}
	else
	{
		ShortRoll = 0;
	}

	if( bArLoading )
	{
		Pitch = TRotator<T>::DecompressAxisFromShort(ShortPitch);
		Yaw	= TRotator<T>::DecompressAxisFromShort(ShortYaw);
		Roll = TRotator<T>::DecompressAxisFromShort(ShortRoll);
	}
}

template<typename T>
UE::Math::TRotator<T> UE::Math::TVector<T>::ToOrientationRotator() const
{
	UE::Math::TRotator<T> R;

	// Find yaw.
	R.Yaw = FMath::RadiansToDegrees(FMath::Atan2(Y, X));

	// Find pitch.
	R.Pitch = FMath::RadiansToDegrees(FMath::Atan2(Z, FMath::Sqrt(X*X + Y*Y)));

	// Find roll.
	R.Roll = 0;

#if ENABLE_NAN_DIAGNOSTIC || (DO_CHECK && !UE_BUILD_SHIPPING)
	if (R.ContainsNaN())
	{
		logOrEnsureNanError(TEXT("TVector::Rotation(): Rotator result %s contains NaN! Input FVector = %s"), *R.ToString(), *this->ToString());
		R = UE::Math::TRotator<T>::ZeroRotator;
	}
#endif

	return R;
}

template<typename T>
UE::Math::TVector<T> UE::Math::TVector<T>::SlerpVectorToDirection(const UE::Math::TVector<T>& V, const UE::Math::TVector<T>& Direction, T Alpha)
{
	using TVector = UE::Math::TVector<T>;
	using TQuat = UE::Math::TQuat<T>;
	
	// Find rotation from A to B
	const TQuat RotationQuat = TQuat::FindBetweenVectors(V, Direction);
	const TVector Axis = RotationQuat.GetRotationAxis();
	const T AngleRads = RotationQuat.GetAngle();

	// Rotate from A toward B using portion of the angle specified by Alpha.
	const TQuat DeltaQuat(Axis, AngleRads * Alpha);
	TVector Result = DeltaQuat.RotateVector(V);
	return Result;
}

template<typename T>
UE::Math::TVector<T> UE::Math::TVector<T>::SlerpNormals(const UE::Math::TVector<T>& NormalA, const UE::Math::TVector<T>& NormalB, T Alpha)
{
	using TVector = UE::Math::TVector<T>;
	using TQuat = UE::Math::TQuat<T>;

	// Find rotation from A to B
	const TQuat RotationQuat = TQuat::FindBetweenNormals(NormalA, NormalB);
	const TVector Axis = RotationQuat.GetRotationAxis();
	const T AngleRads = RotationQuat.GetAngle();

	// Rotate from A toward B using portion of the angle specified by Alpha.
	const TQuat DeltaQuat(Axis, AngleRads * Alpha);
	TVector Result = DeltaQuat.RotateVector(NormalA);
	return Result;
}

template<typename T>
UE::Math::TRotator<T> UE::Math::TVector4<T>::ToOrientationRotator() const
{
	UE::Math::TRotator<T> R;

	// Find yaw.
	R.Yaw = FMath::RadiansToDegrees(FMath::Atan2(Y, X));

	// Find pitch.
	R.Pitch = FMath::RadiansToDegrees(FMath::Atan2(Z, FMath::Sqrt(X*X + Y*Y)));

	// Find roll.
	R.Roll = 0;

#if ENABLE_NAN_DIAGNOSTIC || (DO_CHECK && !UE_BUILD_SHIPPING)
	if (R.ContainsNaN())
	{
		logOrEnsureNanError(TEXT("TVector4::Rotation(): Rotator result %s contains NaN! Input FVector4 = %s"), *R.ToString(), *this->ToString());
		R = UE::Math::TRotator<T>::ZeroRotator;
	}
#endif

	return R;
}


template<typename T>
UE::Math::TQuat<T> UE::Math::TVector<T>::ToOrientationQuat() const
{
	// Essentially an optimized Vector->Rotator->Quat made possible by knowing Roll == 0, and avoiding radians->degrees->radians.
	// This is done to avoid adding any roll (which our API states as a constraint).
	const T YawRad = FMath::Atan2(Y, X);
	const T PitchRad = FMath::Atan2(Z, FMath::Sqrt(X*X + Y*Y));

	const T DIVIDE_BY_2 = 0.5;
	T SP, SY;
	T CP, CY;

	FMath::SinCos(&SP, &CP, PitchRad * DIVIDE_BY_2);
	FMath::SinCos(&SY, &CY, YawRad * DIVIDE_BY_2);

	UE::Math::TQuat<T> RotationQuat;
	RotationQuat.X =  SP*SY;
	RotationQuat.Y = -SP*CY;
	RotationQuat.Z =  CP*SY;
	RotationQuat.W =  CP*CY;
	return RotationQuat;
}

template<typename T>
UE::Math::TQuat<T> UE::Math::TVector4<T>::ToOrientationQuat() const
{
	// Essentially an optimized Vector->Rotator->Quat made possible by knowing Roll == 0, and avoiding radians->degrees->radians.
	// This is done to avoid adding any roll (which our API states as a constraint).
	const T YawRad = FMath::Atan2(Y, X);
	const T PitchRad = FMath::Atan2(Z, FMath::Sqrt(X * X + Y * Y));

	const T DIVIDE_BY_2 = 0.5;
	T SP, SY;
	T CP, CY;

	FMath::SinCos(&SP, &CP, PitchRad * DIVIDE_BY_2);
	FMath::SinCos(&SY, &CY, YawRad * DIVIDE_BY_2);

	UE::Math::TQuat<T> RotationQuat;
	RotationQuat.X = SP * SY;
	RotationQuat.Y = -SP * CY;
	RotationQuat.Z = CP * SY;
	RotationQuat.W = CP * CY;
	return RotationQuat;
}

template <typename T>
UE::Math::TVector<T> FMath::ClosestPointOnLine(const UE::Math::TVector<T>& LineStart, const UE::Math::TVector<T>& LineEnd, const UE::Math::TVector<T>& Point)
{
	// Solve to find alpha along line that is closest point
	// Weisstein, Eric W. "Point-Line Distance--3-Dimensional." From MathWorld--A Switchram Web Resource. http://mathworld.wolfram.com/Point-LineDistance3-Dimensional.html 
	const T A = (LineStart - Point) | (LineEnd - LineStart);
	const T B = (LineEnd - LineStart).SizeSquared();
	// This should be robust to B == 0 (resulting in NaN) because clamp should return 1.
	const T Time = FMath::Clamp<T>(-A/B, 0.f, 1.f);

	// Generate closest point
	UE::Math::TVector<T> ClosestPoint = LineStart + (Time * (LineEnd - LineStart));

	return ClosestPoint;
}

FVector FMath::ClosestPointOnInfiniteLine(const FVector& LineStart, const FVector& LineEnd, const FVector& Point)
{
	const FVector::FReal A = (LineStart - Point) | (LineEnd - LineStart);
	const FVector::FReal B = (LineEnd - LineStart).SizeSquared();
	if (B < UE_SMALL_NUMBER)
	{
		return LineStart;
	}
	const FVector::FReal T = -A/B;

	// Generate closest point
	const FVector ClosestPoint = LineStart + (T * (LineEnd - LineStart));
	return ClosestPoint;
}


template<typename T>
UE::Math::TRotator<T>::TRotator(const UE::Math::TQuat<T>& Quat)
{
	*this = Quat.Rotator();
	DiagnosticCheckNaN();
}


template<typename T>
UE::Math::TVector<T> UE::Math::TRotator<T>::Vector() const
{
	// Extremely large but valid values (or invalid values from uninitialized vars) can cause SinCos to return NaN/Inf, so catch that here. Similar to what is done in FRotator::Quaternion().
#if ENABLE_NAN_DIAGNOSTIC || (DO_CHECK && !UE_BUILD_SHIPPING)
	if (FMath::Abs(Pitch) > UE_FLOAT_NON_FRACTIONAL ||
		FMath::Abs(Yaw  ) > UE_FLOAT_NON_FRACTIONAL ||
		FMath::Abs(Roll ) > UE_FLOAT_NON_FRACTIONAL)
	{
		logOrEnsureNanError(TEXT("FRotator::Vector() provided with unreasonably large input values (%s), possible use of uninitialized variable?"), *ToString());
	}
#endif
	
	// Remove winding and clamp to [-360, 360]
	const T PitchNoWinding = FMath::Fmod(Pitch, (T)360.0);
	const T YawNoWinding = FMath::Fmod(Yaw, (T)360.0);

	T CP, SP, CY, SY;
	FMath::SinCos( &SP, &CP, FMath::DegreesToRadians(PitchNoWinding) );
	FMath::SinCos( &SY, &CY, FMath::DegreesToRadians(YawNoWinding) );
	UE::Math::TVector<T> V = UE::Math::TVector<T>( CP*CY, CP*SY, SP );

	// Error checking
#if ENABLE_NAN_DIAGNOSTIC || (DO_CHECK && !UE_BUILD_SHIPPING)
	if (V.ContainsNaN())
	{
		logOrEnsureNanError(TEXT("FRotator::Vector() resulted in NaN/Inf with input: %s output: %s"), *ToString(), *V.ToString());
		V = UE::Math::TVector<T>::ForwardVector;
	}
#endif

	return V;
}


template<typename T>
UE::Math::TRotator<T> UE::Math::TRotator<T>::GetInverse() const
{
	return Quaternion().Inverse().Rotator();
}


template<>
FQuat4f FRotator3f::Quaternion() const
{
	DiagnosticCheckNaN();

#if PLATFORM_ENABLE_VECTORINTRINSICS
	const VectorRegister4Float Angles = MakeVectorRegisterFloat(Pitch, Yaw, Roll, 0.0f);
	const VectorRegister4Float AnglesNoWinding = VectorMod360(Angles);
	const VectorRegister4Float HalfAngles = VectorMultiply(AnglesNoWinding, GlobalVectorConstants::DEG_TO_RAD_HALF);

	VectorRegister4Float SinAngles, CosAngles;
	VectorSinCos(&SinAngles, &CosAngles, &HalfAngles);

	// Vectorized conversion, measured 20% faster than using scalar version after VectorSinCos.
	// Indices within VectorRegister (for shuffles): P=0, Y=1, R=2
	const VectorRegister4Float SR = VectorReplicate(SinAngles, 2);
	const VectorRegister4Float CR = VectorReplicate(CosAngles, 2);

	const VectorRegister4Float SY_SY_CY_CY_Temp = VectorShuffle(SinAngles, CosAngles, 1, 1, 1, 1);

	const VectorRegister4Float SP_SP_CP_CP = VectorShuffle(SinAngles, CosAngles, 0, 0, 0, 0);
	const VectorRegister4Float SY_CY_SY_CY = VectorShuffle(SY_SY_CY_CY_Temp, SY_SY_CY_CY_Temp, 0, 2, 0, 2);

	const VectorRegister4Float CP_CP_SP_SP = VectorShuffle(CosAngles, SinAngles, 0, 0, 0, 0);
	const VectorRegister4Float CY_SY_CY_SY = VectorShuffle(SY_SY_CY_CY_Temp, SY_SY_CY_CY_Temp, 2, 0, 2, 0);

	const uint32 Neg = uint32(1 << 31);
	const uint32 Pos = uint32(0);
	const VectorRegister4Float SignBitsLeft  = MakeVectorRegister(Pos, Neg, Pos, Pos);
	const VectorRegister4Float SignBitsRight = MakeVectorRegister(Neg, Neg, Neg, Pos);
	const VectorRegister4Float LeftTerm  = VectorBitwiseXor(SignBitsLeft , VectorMultiply(CR, VectorMultiply(SP_SP_CP_CP, SY_CY_SY_CY)));
	const VectorRegister4Float RightTerm = VectorBitwiseXor(SignBitsRight, VectorMultiply(SR, VectorMultiply(CP_CP_SP_SP, CY_SY_CY_SY)));

	const VectorRegister4Float Result = VectorAdd(LeftTerm, RightTerm);	
	FQuat4f RotationQuat = FQuat4f::MakeFromVectorRegister(Result);
#else
	const float DEG_TO_RAD = UE_PI/(180.f);
	const float RADS_DIVIDED_BY_2 = DEG_TO_RAD/2.f;
	float SP, SY, SR;
	float CP, CY, CR;

	const float PitchNoWinding = FMath::Fmod(Pitch, 360.0f);
	const float YawNoWinding = FMath::Fmod(Yaw, 360.0f);
	const float RollNoWinding = FMath::Fmod(Roll, 360.0f);

	FMath::SinCos(&SP, &CP, PitchNoWinding * RADS_DIVIDED_BY_2);
	FMath::SinCos(&SY, &CY, YawNoWinding * RADS_DIVIDED_BY_2);
	FMath::SinCos(&SR, &CR, RollNoWinding * RADS_DIVIDED_BY_2);

	FQuat4f RotationQuat;
	RotationQuat.X =  CR*SP*SY - SR*CP*CY;
	RotationQuat.Y = -CR*SP*CY - SR*CP*SY;
	RotationQuat.Z =  CR*CP*SY - SR*SP*CY;
	RotationQuat.W =  CR*CP*CY + SR*SP*SY;
#endif // PLATFORM_ENABLE_VECTORINTRINSICS

#if ENABLE_NAN_DIAGNOSTIC || DO_CHECK
	// Very large inputs can cause NaN's. Want to catch this here
	if (RotationQuat.ContainsNaN())
	{
		logOrEnsureNanError(TEXT("Invalid input %s to FRotator::Quaternion - generated NaN output: %s"), *ToString(), *RotationQuat.ToString());
		RotationQuat = FQuat4f::Identity;
		FDebug::DumpStackTraceToLog(ELogVerbosity::Warning);
	}
#endif

	return RotationQuat;
}


template<>
FQuat4d FRotator3d::Quaternion() const
{
	DiagnosticCheckNaN();

#if PLATFORM_ENABLE_VECTORINTRINSICS
	const VectorRegister4Double Angles = MakeVectorRegisterDouble(Pitch, Yaw, Roll, 0.0);
	const VectorRegister4Double AnglesNoWinding = VectorMod360(Angles);
	const VectorRegister4Double HalfAngles = VectorMultiply(AnglesNoWinding, GlobalVectorConstants::DOUBLE_DEG_TO_RAD_HALF);

	VectorRegister4Double SinAngles, CosAngles;
	VectorSinCos(&SinAngles, &CosAngles, &HalfAngles);

	// Vectorized conversion, measured 20% faster than using scalar version after VectorSinCos.
	// Indices within VectorRegister (for shuffles): P=0, Y=1, R=2
	const VectorRegister4Double SR = VectorReplicate(SinAngles, 2);
	const VectorRegister4Double CR = VectorReplicate(CosAngles, 2);

	const VectorRegister4Double SY_SY_CY_CY_Temp = VectorShuffle(SinAngles, CosAngles, 1, 1, 1, 1);

	const VectorRegister4Double SP_SP_CP_CP = VectorShuffle(SinAngles, CosAngles, 0, 0, 0, 0);
	const VectorRegister4Double SY_CY_SY_CY = VectorShuffle(SY_SY_CY_CY_Temp, SY_SY_CY_CY_Temp, 0, 2, 0, 2);

	const VectorRegister4Double CP_CP_SP_SP = VectorShuffle(CosAngles, SinAngles, 0, 0, 0, 0);
	const VectorRegister4Double CY_SY_CY_SY = VectorShuffle(SY_SY_CY_CY_Temp, SY_SY_CY_CY_Temp, 2, 0, 2, 0);

	const uint64 Neg = uint64(1) << 63;
	const uint64 Pos = uint64(0);
	const VectorRegister4Double SignBitsLeft = MakeVectorRegisterDoubleMask(Pos, Neg, Pos, Pos);
	const VectorRegister4Double SignBitsRight = MakeVectorRegisterDoubleMask(Neg, Neg, Neg, Pos);
	const VectorRegister4Double LeftTerm = VectorBitwiseXor(SignBitsLeft, VectorMultiply(CR, VectorMultiply(SP_SP_CP_CP, SY_CY_SY_CY)));
	const VectorRegister4Double RightTerm = VectorBitwiseXor(SignBitsRight, VectorMultiply(SR, VectorMultiply(CP_CP_SP_SP, CY_SY_CY_SY)));

	const VectorRegister4Double Result = VectorAdd(LeftTerm, RightTerm);
	FQuat4d RotationQuat = FQuat4d::MakeFromVectorRegister(Result);
#else
	const double DEG_TO_RAD = UE_DOUBLE_PI / (180.0);
	const double RADS_DIVIDED_BY_2 = DEG_TO_RAD / 2.0;
	double SP, SY, SR;
	double CP, CY, CR;

	const double PitchNoWinding = FMath::Fmod(Pitch, 360.0);
	const double YawNoWinding = FMath::Fmod(Yaw, 360.0);
	const double RollNoWinding = FMath::Fmod(Roll, 360.0);

	FMath::SinCos(&SP, &CP, PitchNoWinding * RADS_DIVIDED_BY_2);
	FMath::SinCos(&SY, &CY, YawNoWinding * RADS_DIVIDED_BY_2);
	FMath::SinCos(&SR, &CR, RollNoWinding * RADS_DIVIDED_BY_2);

	FQuat4d RotationQuat;
	RotationQuat.X = CR * SP * SY - SR * CP * CY;
	RotationQuat.Y = -CR * SP * CY - SR * CP * SY;
	RotationQuat.Z = CR * CP * SY - SR * SP * CY;
	RotationQuat.W = CR * CP * CY + SR * SP * SY;
#endif // PLATFORM_ENABLE_VECTORINTRINSICS

#if ENABLE_NAN_DIAGNOSTIC || DO_CHECK
	// Very large inputs can cause NaN's. Want to catch this here
	if (RotationQuat.ContainsNaN())
	{
		logOrEnsureNanError(TEXT("Invalid input %s to FRotator::Quaternion - generated NaN output: %s"), *ToString(), *RotationQuat.ToString());
		RotationQuat = FQuat4d::Identity;
		FDebug::DumpStackTraceToLog(ELogVerbosity::Warning);
	}
#endif

	return RotationQuat;
}


template<typename T>
UE::Math::TVector<T> UE::Math::TRotator<T>::Euler() const
{
	return UE::Math::TVector<T>( Roll, Pitch, Yaw );
}

template<typename T>
UE::Math::TRotator<T> UE::Math::TRotator<T>::MakeFromEuler(const UE::Math::TVector<T>& Euler)
{
	return UE::Math::TRotator<T>(Euler.Y, Euler.Z, Euler.X);
}

template<typename T>
UE::Math::TVector<T> UE::Math::TRotator<T>::UnrotateVector(const UE::Math::TVector<T>& V) const
{
	return UE::Math::TRotationMatrix<T>(*this).GetTransposed().TransformVector( V );
}	

template<typename T>
UE::Math::TVector<T> UE::Math::TRotator<T>::RotateVector(const UE::Math::TVector<T>& V) const
{
	return UE::Math::TRotationMatrix<T>(*this).TransformVector( V );
}	

template<typename T>
void UE::Math::TRotator<T>::GetWindingAndRemainder(UE::Math::TRotator<T>& Winding, UE::Math::TRotator<T>& Remainder) const
{
	//// YAW
	Remainder.Yaw = NormalizeAxis(Yaw);

	Winding.Yaw = Yaw - Remainder.Yaw;

	//// PITCH
	Remainder.Pitch = NormalizeAxis(Pitch);

	Winding.Pitch = Pitch - Remainder.Pitch;

	//// ROLL
	Remainder.Roll = NormalizeAxis(Roll);

	Winding.Roll = Roll - Remainder.Roll;
}


template<typename T>
UE::Math::TRotator<T> UE::Math::TMatrix<T>::Rotator() const
{
	using TRotator = UE::Math::TRotator<T>;
	using TVector = UE::Math::TVector<T>;
	const TVector		XAxis	= GetScaledAxis( EAxis::X );
	const TVector		YAxis	= GetScaledAxis( EAxis::Y );
	const TVector		ZAxis	= GetScaledAxis( EAxis::Z );
	const T RadToDeg = T(180.0 / UE_DOUBLE_PI);

	TRotator Rotator	= TRotator(
									FMath::Atan2( XAxis.Z, FMath::Sqrt(FMath::Square(XAxis.X)+FMath::Square(XAxis.Y)) ) * RadToDeg,
									FMath::Atan2( XAxis.Y, XAxis.X ) * RadToDeg,
									0 
								);
	
	const TVector	SYAxis	= (TVector)UE::Math::TRotationMatrix<T>( Rotator ).GetScaledAxis( EAxis::Y );
	Rotator.Roll		= FMath::Atan2( ZAxis | SYAxis, YAxis | SYAxis ) * RadToDeg;

	Rotator.DiagnosticCheckNaN();
	return Rotator;
}

template<>
FQuat4f FMatrix44f::ToQuat() const
{
	return FQuat4f(*this);
}

template<>
FQuat4d FMatrix44d::ToQuat() const
{
	return FQuat4d(*this);
}


namespace UE
{
namespace Math
{
	//////////////////////////////////////////////////////////////////////////
	// FQuat

	template<>
	FRotator3f FQuat4f::Rotator() const
	{
		DiagnosticCheckNaN();
		const float SingularityTest = Z * X - W * Y;
		const float YawY = 2.f * (W * Z + X * Y);
		const float YawX = (1.f - 2.f * (FMath::Square(Y) + FMath::Square(Z)));

		// reference 
		// http://en.wikipedia.org/wiki/Conversion_between_quaternions_and_Euler_angles
		// http://www.euclideanspace.com/maths/geometry/rotations/conversions/quaternionToEuler/

		// this value was found from experience, the above websites recommend different values
		// but that isn't the case for us, so I went through different testing, and finally found the case 
		// where both of world lives happily. 
		const float SINGULARITY_THRESHOLD = 0.4999995f;
		const float RAD_TO_DEG = (180.f / UE_PI);
		float Pitch, Yaw, Roll;

		if (SingularityTest < -SINGULARITY_THRESHOLD)
		{
			Pitch = -90.f;
			Yaw = FRotator3f::NormalizeAxis(-2.0f * FMath::Atan2(X, W) * RAD_TO_DEG);
			Roll = 0.f;
		}
		else if (SingularityTest > SINGULARITY_THRESHOLD)
		{
			Pitch = 90.f;
			Yaw = FRotator3f::NormalizeAxis(2.0f * FMath::Atan2(X, W) * RAD_TO_DEG);
			Roll = 0.f;
		}
		else
		{
			Pitch = (FMath::FastAsin(2.f * SingularityTest) * RAD_TO_DEG);
			Yaw = (FMath::Atan2(YawY, YawX) * RAD_TO_DEG);
			Roll = (FMath::Atan2(-2.f * (W*X + Y*Z), (1.f - 2.f * (FMath::Square(X) + FMath::Square(Y)))) * RAD_TO_DEG);
		}

		FRotator3f RotatorFromQuat = FRotator3f(Pitch, Yaw, Roll);

#if ENABLE_NAN_DIAGNOSTIC
		if (RotatorFromQuat.ContainsNaN())
		{
			logOrEnsureNanError(TEXT("TQuat<T>::Rotator(): Rotator result %s contains NaN! Quat = %s, YawY = %.9f, YawX = %.9f"), *RotatorFromQuat.ToString(), *this->ToString(), YawY, YawX);
			RotatorFromQuat = FRotator3f::ZeroRotator;
		}
#endif

		return RotatorFromQuat;
	}

	template<>
	FRotator3d FQuat4d::Rotator() const
	{
		DiagnosticCheckNaN();
		const double SingularityTest = Z * X - W * Y;
		const double YawY = 2.0 * (W * Z + X * Y);
		const double YawX = (1.0 - 2.0 * (FMath::Square(Y) + FMath::Square(Z)));

		const double SINGULARITY_THRESHOLD = 0.4999995;
		const double RAD_TO_DEG = (180.0 / UE_DOUBLE_PI);
		double Pitch, Yaw, Roll;

		if (SingularityTest < -SINGULARITY_THRESHOLD)
		{
			Pitch = -90.0;
			Yaw = FRotator3d::NormalizeAxis(-2.0 * FMath::Atan2(X, W) * RAD_TO_DEG);
			Roll = 0.0;
		}
		else if (SingularityTest > SINGULARITY_THRESHOLD)
		{
			Pitch = 90.0;
			Yaw = FRotator3d::NormalizeAxis(2.0 * FMath::Atan2(X, W) * RAD_TO_DEG);
			Roll = 0.0;
		}
		else
		{
			Pitch = (FMath::Asin(2.0 * SingularityTest) * RAD_TO_DEG); // Note: not FastAsin like float implementation
			Yaw = (FMath::Atan2(YawY, YawX) * RAD_TO_DEG);
			Roll = (FMath::Atan2(-2.0 * (W * X + Y * Z), (1.0 - 2.0 * (FMath::Square(X) + FMath::Square(Y)))) * RAD_TO_DEG);
		}

		FRotator3d RotatorFromQuat = FRotator3d(Pitch, Yaw, Roll);

#if ENABLE_NAN_DIAGNOSTIC
		if (RotatorFromQuat.ContainsNaN())
		{
			logOrEnsureNanError(TEXT("TQuat<T>::Rotator(): Rotator result %s contains NaN! Quat = %s, YawY = %.9f, YawX = %.9f"), *RotatorFromQuat.ToString(), *this->ToString(), YawY, YawX);
			RotatorFromQuat = FRotator3d::ZeroRotator;
		}
#endif

		return RotatorFromQuat;
	}

	template<typename T>
	TMatrix<T> TQuat<T>::operator*(const TMatrix<T>& M) const
	{
		TMatrix<T> Result;
#if PLATFORM_ENABLE_VECTORINTRINSICS
		const TQuat<T>::QuatVectorRegister This = VectorLoadAligned(this);
		const TQuat<T>::QuatVectorRegister Inv = VectorQuaternionInverse(This);
		for (int32 I = 0; I < 4; ++I)
		{
			TQuat<T>::QuatVectorRegister VT, VR;
			const TQuat<T>::QuatVectorRegister VQ = VectorLoad(M.M[I]);
			VT = VectorQuaternionMultiply2(This, VQ);
			VR = VectorQuaternionMultiply2(VT, Inv);
			VectorStore(VR, Result.M[I]);
		}
#else
		const TQuat<T> Inv = Inverse();
		for (int32 I = 0; I < 4; ++I)
		{
			TQuat<T> VT, VR;
			const TQuat<T> VQ(M.M[I][0], M.M[I][1], M.M[I][2], M.M[I][3]);
			VectorQuaternionMultiply(&VT, this, &VQ);
			VectorQuaternionMultiply(&VR, &VT, &Inv);
			Result.M[I][0] = VR.X;
			Result.M[I][1] = VR.Y;
			Result.M[I][2] = VR.Z;
			Result.M[I][3] = VR.W;
		}
#endif

		return Result;
	}

	template<typename T>
	void TQuat<T>::ToMatrix(TMatrix<T>& R) const
	{
		// Note: copied and modified from TQuatRotationTranslationMatrix<> mainly to avoid circular dependency.
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST) && WITH_EDITORONLY_DATA
		// Make sure Quaternion is normalized
		ensure(IsNormalized());
#endif
		const T x2 = X + X;    const T y2 = Y + Y;    const T z2 = Z + Z;
		const T xx = X * x2;   const T xy = X * y2;   const T xz = X * z2;
		const T yy = Y * y2;   const T yz = Y * z2;   const T zz = Z * z2;
		const T wx = W * x2;   const T wy = W * y2;   const T wz = W * z2;

		R.M[0][0] = 1.0f - (yy + zz);	R.M[1][0] = xy - wz;				R.M[2][0] = xz + wy;			R.M[3][0] = 0.0f;
		R.M[0][1] = xy + wz;			R.M[1][1] = 1.0f - (xx + zz);		R.M[2][1] = yz - wx;			R.M[3][1] = 0.0f;
		R.M[0][2] = xz - wy;			R.M[1][2] = yz + wx;				R.M[2][2] = 1.0f - (xx + yy);	R.M[3][2] = 0.0f;
		R.M[0][3] = 0.0f;				R.M[1][3] = 0.0f;					R.M[2][3] = 0.0f;				R.M[3][3] = 1.0f;
	}

	template<typename T>
	TQuat<T> TQuat<T>::MakeFromEuler(const TVector<T>& Euler)
	{
		return TQuat<T>(TRotator<T>::MakeFromEuler(Euler));
	}

	template<typename T>
	void TQuat<T>::ToSwingTwist(const TVector<T>& InTwistAxis, TQuat<T>& OutSwing, TQuat<T>& OutTwist) const
	{
		// Vector part projected onto twist axis
		TVector<T> Projection = TVector<T>::DotProduct(InTwistAxis, TVector<T>(X, Y, Z)) * InTwistAxis;

		// Twist quaternion
		OutTwist = TQuat<T>((FReal)Projection.X, (FReal)Projection.Y, (FReal)Projection.Z, W);

		// Singularity close to 180deg
		if (OutTwist.SizeSquared() == 0.0f)
		{
			OutTwist = TQuat<T>::Identity;
		}
		else
		{
			OutTwist.Normalize();
		}

		// Set swing
		OutSwing = *this * OutTwist.Inverse();
	}

	template<typename T>
	T TQuat<T>::GetTwistAngle(const TVector<T>& TwistAxis) const
	{
		T XYZ = (T)TVector<T>::DotProduct(TwistAxis, TVector<T>(X, Y, Z));
		return FMath::UnwindRadians((T)2.0f * FMath::Atan2(XYZ, W));
	}
	
} // namespace UE::Math
}

	
/**
 * Calculate the inverse of an FMatrix44.
 *
 * @param DstMatrix		FMatrix44 pointer to where the result should be stored
 * @param SrcMatrix		FMatrix44 pointer to the Matrix to be inversed
 * @return bool			returns false if matrix is not invertable and stores identity 
 *
 * Src == Dst is allowed
 */
 // scalar implementation of MatrixInverse is here, used as the fallback for VectorMatrixInverse
bool FMath::MatrixInverse(FMatrix44d* DstMatrix, const FMatrix44d* SrcMatrix)
{
	typedef double Double4x4[4][4];
	const Double4x4& M = *((const Double4x4*)SrcMatrix);
	Double4x4 Result;
	double Det[4];
	Double4x4 Tmp;

	Tmp[0][0] = M[2][2] * M[3][3] - M[2][3] * M[3][2];
	Tmp[0][1] = M[1][2] * M[3][3] - M[1][3] * M[3][2];
	Tmp[0][2] = M[1][2] * M[2][3] - M[1][3] * M[2][2];

	Tmp[1][0] = M[2][2] * M[3][3] - M[2][3] * M[3][2];
	Tmp[1][1] = M[0][2] * M[3][3] - M[0][3] * M[3][2];
	Tmp[1][2] = M[0][2] * M[2][3] - M[0][3] * M[2][2];

	Tmp[2][0] = M[1][2] * M[3][3] - M[1][3] * M[3][2];
	Tmp[2][1] = M[0][2] * M[3][3] - M[0][3] * M[3][2];
	Tmp[2][2] = M[0][2] * M[1][3] - M[0][3] * M[1][2];

	Tmp[3][0] = M[1][2] * M[2][3] - M[1][3] * M[2][2];
	Tmp[3][1] = M[0][2] * M[2][3] - M[0][3] * M[2][2];
	Tmp[3][2] = M[0][2] * M[1][3] - M[0][3] * M[1][2];

	Det[0] = M[1][1] * Tmp[0][0] - M[2][1] * Tmp[0][1] + M[3][1] * Tmp[0][2];
	Det[1] = M[0][1] * Tmp[1][0] - M[2][1] * Tmp[1][1] + M[3][1] * Tmp[1][2];
	Det[2] = M[0][1] * Tmp[2][0] - M[1][1] * Tmp[2][1] + M[3][1] * Tmp[2][2];
	Det[3] = M[0][1] * Tmp[3][0] - M[1][1] * Tmp[3][1] + M[2][1] * Tmp[3][2];

	const double Determinant = M[0][0] * Det[0] - M[1][0] * Det[1] + M[2][0] * Det[2] - M[3][0] * Det[3];

	if ( Determinant == 0.0 || !FMath::IsFinite(Determinant) )
	{
		*DstMatrix = FMatrix44d::Identity;
		return false;
	}

	const double	RDet = 1.0 / Determinant;

	Result[0][0] = RDet * Det[0];
	Result[0][1] = -RDet * Det[1];
	Result[0][2] = RDet * Det[2];
	Result[0][3] = -RDet * Det[3];
	Result[1][0] = -RDet * (M[1][0] * Tmp[0][0] - M[2][0] * Tmp[0][1] + M[3][0] * Tmp[0][2]);
	Result[1][1] = RDet * (M[0][0] * Tmp[1][0] - M[2][0] * Tmp[1][1] + M[3][0] * Tmp[1][2]);
	Result[1][2] = -RDet * (M[0][0] * Tmp[2][0] - M[1][0] * Tmp[2][1] + M[3][0] * Tmp[2][2]);
	Result[1][3] = RDet * (M[0][0] * Tmp[3][0] - M[1][0] * Tmp[3][1] + M[2][0] * Tmp[3][2]);
	Result[2][0] = RDet * (
		M[1][0] * (M[2][1] * M[3][3] - M[2][3] * M[3][1]) -
		M[2][0] * (M[1][1] * M[3][3] - M[1][3] * M[3][1]) +
		M[3][0] * (M[1][1] * M[2][3] - M[1][3] * M[2][1])
		);
	Result[2][1] = -RDet * (
		M[0][0] * (M[2][1] * M[3][3] - M[2][3] * M[3][1]) -
		M[2][0] * (M[0][1] * M[3][3] - M[0][3] * M[3][1]) +
		M[3][0] * (M[0][1] * M[2][3] - M[0][3] * M[2][1])
		);
	Result[2][2] = RDet * (
		M[0][0] * (M[1][1] * M[3][3] - M[1][3] * M[3][1]) -
		M[1][0] * (M[0][1] * M[3][3] - M[0][3] * M[3][1]) +
		M[3][0] * (M[0][1] * M[1][3] - M[0][3] * M[1][1])
		);
	Result[2][3] = -RDet * (
		M[0][0] * (M[1][1] * M[2][3] - M[1][3] * M[2][1]) -
		M[1][0] * (M[0][1] * M[2][3] - M[0][3] * M[2][1]) +
		M[2][0] * (M[0][1] * M[1][3] - M[0][3] * M[1][1])
		);
	Result[3][0] = -RDet * (
		M[1][0] * (M[2][1] * M[3][2] - M[2][2] * M[3][1]) -
		M[2][0] * (M[1][1] * M[3][2] - M[1][2] * M[3][1]) +
		M[3][0] * (M[1][1] * M[2][2] - M[1][2] * M[2][1])
		);
	Result[3][1] = RDet * (
		M[0][0] * (M[2][1] * M[3][2] - M[2][2] * M[3][1]) -
		M[2][0] * (M[0][1] * M[3][2] - M[0][2] * M[3][1]) +
		M[3][0] * (M[0][1] * M[2][2] - M[0][2] * M[2][1])
		);
	Result[3][2] = -RDet * (
		M[0][0] * (M[1][1] * M[3][2] - M[1][2] * M[3][1]) -
		M[1][0] * (M[0][1] * M[3][2] - M[0][2] * M[3][1]) +
		M[3][0] * (M[0][1] * M[1][2] - M[0][2] * M[1][1])
		);
	Result[3][3] = RDet * (
		M[0][0] * (M[1][1] * M[2][2] - M[1][2] * M[2][1]) -
		M[1][0] * (M[0][1] * M[2][2] - M[0][2] * M[2][1]) +
		M[2][0] * (M[0][1] * M[1][2] - M[0][2] * M[1][1])
		);

	memcpy(DstMatrix, &Result, 16 * sizeof(double));
	return true;
}

bool FMath::MatrixInverse(FMatrix44f* DstMatrix, const FMatrix44f* SrcMatrix)
{
	typedef float Float4x4[4][4];
	const Float4x4& M = *((const Float4x4*)SrcMatrix);
	Float4x4 Result;
	float Det[4];
	Float4x4 Tmp;

	Tmp[0][0] = M[2][2] * M[3][3] - M[2][3] * M[3][2];
	Tmp[0][1] = M[1][2] * M[3][3] - M[1][3] * M[3][2];
	Tmp[0][2] = M[1][2] * M[2][3] - M[1][3] * M[2][2];

	Tmp[1][0] = M[2][2] * M[3][3] - M[2][3] * M[3][2];
	Tmp[1][1] = M[0][2] * M[3][3] - M[0][3] * M[3][2];
	Tmp[1][2] = M[0][2] * M[2][3] - M[0][3] * M[2][2];

	Tmp[2][0] = M[1][2] * M[3][3] - M[1][3] * M[3][2];
	Tmp[2][1] = M[0][2] * M[3][3] - M[0][3] * M[3][2];
	Tmp[2][2] = M[0][2] * M[1][3] - M[0][3] * M[1][2];

	Tmp[3][0] = M[1][2] * M[2][3] - M[1][3] * M[2][2];
	Tmp[3][1] = M[0][2] * M[2][3] - M[0][3] * M[2][2];
	Tmp[3][2] = M[0][2] * M[1][3] - M[0][3] * M[1][2];

	Det[0] = M[1][1] * Tmp[0][0] - M[2][1] * Tmp[0][1] + M[3][1] * Tmp[0][2];
	Det[1] = M[0][1] * Tmp[1][0] - M[2][1] * Tmp[1][1] + M[3][1] * Tmp[1][2];
	Det[2] = M[0][1] * Tmp[2][0] - M[1][1] * Tmp[2][1] + M[3][1] * Tmp[2][2];
	Det[3] = M[0][1] * Tmp[3][0] - M[1][1] * Tmp[3][1] + M[2][1] * Tmp[3][2];

	const float Determinant = M[0][0] * Det[0] - M[1][0] * Det[1] + M[2][0] * Det[2] - M[3][0] * Det[3];
	
	if ( Determinant == 0.0f || !FMath::IsFinite(Determinant) )
	{
		*DstMatrix = FMatrix44f::Identity;
		return false;
	}

	const float	RDet = 1.0f / Determinant;

	Result[0][0] = RDet * Det[0];
	Result[0][1] = -RDet * Det[1];
	Result[0][2] = RDet * Det[2];
	Result[0][3] = -RDet * Det[3];
	Result[1][0] = -RDet * (M[1][0] * Tmp[0][0] - M[2][0] * Tmp[0][1] + M[3][0] * Tmp[0][2]);
	Result[1][1] = RDet * (M[0][0] * Tmp[1][0] - M[2][0] * Tmp[1][1] + M[3][0] * Tmp[1][2]);
	Result[1][2] = -RDet * (M[0][0] * Tmp[2][0] - M[1][0] * Tmp[2][1] + M[3][0] * Tmp[2][2]);
	Result[1][3] = RDet * (M[0][0] * Tmp[3][0] - M[1][0] * Tmp[3][1] + M[2][0] * Tmp[3][2]);
	Result[2][0] = RDet * (
		M[1][0] * (M[2][1] * M[3][3] - M[2][3] * M[3][1]) -
		M[2][0] * (M[1][1] * M[3][3] - M[1][3] * M[3][1]) +
		M[3][0] * (M[1][1] * M[2][3] - M[1][3] * M[2][1])
		);
	Result[2][1] = -RDet * (
		M[0][0] * (M[2][1] * M[3][3] - M[2][3] * M[3][1]) -
		M[2][0] * (M[0][1] * M[3][3] - M[0][3] * M[3][1]) +
		M[3][0] * (M[0][1] * M[2][3] - M[0][3] * M[2][1])
		);
	Result[2][2] = RDet * (
		M[0][0] * (M[1][1] * M[3][3] - M[1][3] * M[3][1]) -
		M[1][0] * (M[0][1] * M[3][3] - M[0][3] * M[3][1]) +
		M[3][0] * (M[0][1] * M[1][3] - M[0][3] * M[1][1])
		);
	Result[2][3] = -RDet * (
		M[0][0] * (M[1][1] * M[2][3] - M[1][3] * M[2][1]) -
		M[1][0] * (M[0][1] * M[2][3] - M[0][3] * M[2][1]) +
		M[2][0] * (M[0][1] * M[1][3] - M[0][3] * M[1][1])
		);
	Result[3][0] = -RDet * (
		M[1][0] * (M[2][1] * M[3][2] - M[2][2] * M[3][1]) -
		M[2][0] * (M[1][1] * M[3][2] - M[1][2] * M[3][1]) +
		M[3][0] * (M[1][1] * M[2][2] - M[1][2] * M[2][1])
		);
	Result[3][1] = RDet * (
		M[0][0] * (M[2][1] * M[3][2] - M[2][2] * M[3][1]) -
		M[2][0] * (M[0][1] * M[3][2] - M[0][2] * M[3][1]) +
		M[3][0] * (M[0][1] * M[2][2] - M[0][2] * M[2][1])
		);
	Result[3][2] = -RDet * (
		M[0][0] * (M[1][1] * M[3][2] - M[1][2] * M[3][1]) -
		M[1][0] * (M[0][1] * M[3][2] - M[0][2] * M[3][1]) +
		M[3][0] * (M[0][1] * M[1][2] - M[0][2] * M[1][1])
		);
	Result[3][3] = RDet * (
		M[0][0] * (M[1][1] * M[2][2] - M[1][2] * M[2][1]) -
		M[1][0] * (M[0][1] * M[2][2] - M[0][2] * M[2][1]) +
		M[2][0] * (M[0][1] * M[1][2] - M[0][2] * M[1][1])
		);

	memcpy(DstMatrix, &Result, 16 * sizeof(float));
	return true;
}




/*-----------------------------------------------------------------------------
	Swept-Box vs Box test.
-----------------------------------------------------------------------------*/

/* Line-extent/Box Test Util */
bool FMath::LineExtentBoxIntersection(const FBox& inBox, 
								 const FVector& Start, 
								 const FVector& End,
								 const FVector& Extent,
								 FVector& HitLocation,
								 FVector& HitNormal,
								 float& HitTime)
{
	FBox box = inBox;
	box.Max.X += Extent.X;
	box.Max.Y += Extent.Y;
	box.Max.Z += Extent.Z;
	
	box.Min.X -= Extent.X;
	box.Min.Y -= Extent.Y;
	box.Min.Z -= Extent.Z;

	const FVector Dir = (End - Start);
	
	FVector	Time;
	bool	Inside = 1;
	float   faceDir[3] = {1, 1, 1};
	
	/////////////// X
	if(Start.X < box.Min.X)
	{
		if(Dir.X <= 0.0f)
			return 0;
		else
		{
			Inside = 0;
			faceDir[0] = -1;
			Time.X = (box.Min.X - Start.X) / Dir.X;
		}
	}
	else if(Start.X > box.Max.X)
	{
		if(Dir.X >= 0.0f)
			return 0;
		else
		{
			Inside = 0;
			Time.X = (box.Max.X - Start.X) / Dir.X;
		}
	}
	else
		Time.X = 0.0f;
	
	/////////////// Y
	if(Start.Y < box.Min.Y)
	{
		if(Dir.Y <= 0.0f)
			return 0;
		else
		{
			Inside = 0;
			faceDir[1] = -1;
			Time.Y = (box.Min.Y - Start.Y) / Dir.Y;
		}
	}
	else if(Start.Y > box.Max.Y)
	{
		if(Dir.Y >= 0.0f)
			return 0;
		else
		{
			Inside = 0;
			Time.Y = (box.Max.Y - Start.Y) / Dir.Y;
		}
	}
	else
		Time.Y = 0.0f;
	
	/////////////// Z
	if(Start.Z < box.Min.Z)
	{
		if(Dir.Z <= 0.0f)
			return 0;
		else
		{
			Inside = 0;
			faceDir[2] = -1;
			Time.Z = (box.Min.Z - Start.Z) / Dir.Z;
		}
	}
	else if(Start.Z > box.Max.Z)
	{
		if(Dir.Z >= 0.0f)
			return 0;
		else
		{
			Inside = 0;
			Time.Z = (box.Max.Z - Start.Z) / Dir.Z;
		}
	}
	else
		Time.Z = 0.0f;
	
	// If the line started inside the box (ie. player started in contact with the fluid)
	if(Inside)
	{
		HitLocation = Start;
		HitNormal = FVector(0, 0, 1);
		HitTime = 0;
		return 1;
	}
	// Otherwise, calculate when hit occured
	else
	{	
		if(Time.Y > Time.Z)
		{
			HitTime = static_cast<std::remove_reference_t<decltype(HitTime)>>(Time.Y);	// LWC_TODO: Remove decltype
			HitNormal = FVector(0, faceDir[1], 0);
		}
		else
		{
			HitTime = static_cast<std::remove_reference_t<decltype(HitTime)>>(Time.Z);
			HitNormal = FVector(0, 0, faceDir[2]);
		}
		
		if(Time.X > HitTime)
		{
			HitTime = static_cast<std::remove_reference_t<decltype(HitTime)>>(Time.X);
			HitNormal = FVector(faceDir[0], 0, 0);
		}
		
		if(HitTime >= 0.0f && HitTime <= 1.0f)
		{
			HitLocation = Start + Dir * HitTime;
			const float BOX_SIDE_THRESHOLD = 0.1f;
			if(	HitLocation.X > box.Min.X - BOX_SIDE_THRESHOLD && HitLocation.X < box.Max.X + BOX_SIDE_THRESHOLD &&
				HitLocation.Y > box.Min.Y - BOX_SIDE_THRESHOLD && HitLocation.Y < box.Max.Y + BOX_SIDE_THRESHOLD &&
				HitLocation.Z > box.Min.Z - BOX_SIDE_THRESHOLD && HitLocation.Z < box.Max.Z + BOX_SIDE_THRESHOLD)
			{				
				return 1;
			}
		}
		
		return 0;
	}
}

float FLinearColor::EvaluateBezier(const FLinearColor* ControlPoints, int32 NumPoints, TArray<FLinearColor>& OutPoints)
{
	check( ControlPoints );
	check( NumPoints >= 2 );

	// var q is the change in t between successive evaluations.
	const float q = 1.f/(float)(NumPoints-1); // q is dependent on the number of GAPS = POINTS-1

	// recreate the names used in the derivation
	const FLinearColor& P0 = ControlPoints[0];
	const FLinearColor& P1 = ControlPoints[1];
	const FLinearColor& P2 = ControlPoints[2];
	const FLinearColor& P3 = ControlPoints[3];

	// coefficients of the cubic polynomial that we're FDing -
	const FLinearColor a = P0;
	const FLinearColor b = 3*(P1-P0);
	const FLinearColor c = 3*(P2-2*P1+P0);
	const FLinearColor d = P3-3*P2+3*P1-P0;

	// initial values of the poly and the 3 diffs -
	FLinearColor S  = a;						// the poly value
	FLinearColor U  = b*q + c*q*q + d*q*q*q;	// 1st order diff (quadratic)
	FLinearColor V  = 2*c*q*q + 6*d*q*q*q;	// 2nd order diff (linear)
	FLinearColor W  = 6*d*q*q*q;				// 3rd order diff (constant)

	// Path length.
	float Length = 0.f;

	FLinearColor OldPos = P0;
	OutPoints.Add( P0 );	// first point on the curve is always P0.

	for( int32 i = 1 ; i < NumPoints ; ++i )
	{
		// calculate the next value and update the deltas
		S += U;			// update poly value
		U += V;			// update 1st order diff value
		V += W;			// update 2st order diff value
		// 3rd order diff is constant => no update needed.

		// Update Length.
		Length += FLinearColor::Dist( S, OldPos );
		OldPos  = S;

		OutPoints.Add( S );
	}

	// Return path length as experienced in sequence (linear interpolation between points).
	return Length;
}

namespace UE
{
namespace Math
{
	template<typename T>
	TQuat<T> TQuat<T>::Slerp_NotNormalized(const TQuat<T>& Quat1, const TQuat<T>& Quat2, T Slerp)
	{
		// Get cosine of angle between quats.
		T RawCosom =
			Quat1.X * Quat2.X +
			Quat1.Y * Quat2.Y +
			Quat1.Z * Quat2.Z +
			Quat1.W * Quat2.W;

		// Unaligned quats - compensate, results in taking shorter route.
		const T Sign = FMath::FloatSelect(RawCosom, static_cast<T>(1.0), static_cast<T>(-1.0));
		RawCosom *= Sign;
		
		T Scale0 = static_cast<T>(1.0) - Slerp;
		T Scale1 = Slerp * Sign;
		
		if (RawCosom < static_cast<T>(0.9999))
		{
			const T Omega = FMath::Acos(RawCosom);
			const T InvSin = static_cast<T>(1.0) / FMath::Sin(Omega);
			Scale0 = FMath::Sin(Scale0 * Omega) * InvSin;
			Scale1 = FMath::Sin(Scale1 * Omega) * InvSin;
		}
		
		return TQuat<T>(
			Scale0 * Quat1.X + Scale1 * Quat2.X,
			Scale0 * Quat1.Y + Scale1 * Quat2.Y,
			Scale0 * Quat1.Z + Scale1 * Quat2.Z,
			Scale0 * Quat1.W + Scale1 * Quat2.W);
	}

	template<typename T>
	TQuat<T> TQuat<T>::SlerpFullPath_NotNormalized(const TQuat<T>&quat1, const TQuat<T>&quat2, T Alpha )
	{
		const T CosAngle = FMath::Clamp(quat1 | quat2, T(-1.f), T(1.f));
		const T Angle = FMath::Acos(CosAngle);

		//UE_LOGF(LogUnrealMath, Log,  "CosAngle: %f Angle: %f", CosAngle, Angle );

		if ( FMath::Abs(Angle) < T(UE_KINDA_SMALL_NUMBER))
		{
			return quat1;
		}

		const T SinAngle = FMath::Sin(Angle);
		const T InvSinAngle = T(1.f)/SinAngle;

		const T Scale0 = FMath::Sin((1.0f-Alpha)*Angle)*InvSinAngle;
		const T Scale1 = FMath::Sin(Alpha*Angle)*InvSinAngle;

		return quat1*Scale0 + quat2*Scale1;
	}

	template<typename T>
	TQuat<T> TQuat<T>::Squad(const TQuat<T>& quat1, const TQuat<T>& tang1, const TQuat<T>& quat2, const TQuat<T>& tang2, T Alpha)
	{
		// Always slerp along the short path from quat1 to quat2 to prevent axis flipping.
		// This approach is taken by OGRE engine, amongst others.
		const TQuat<T> Q1 = TQuat::Slerp_NotNormalized(quat1, quat2, Alpha);
		const TQuat<T> Q2 = TQuat::SlerpFullPath_NotNormalized(tang1, tang2, Alpha);
		const TQuat<T> Result = TQuat::SlerpFullPath(Q1, Q2, 2.f * Alpha * (1.f - Alpha));

		return Result;
	}

	template<typename T>
	TQuat<T> TQuat<T>::SquadFullPath(const TQuat<T>& quat1, const TQuat<T>& tang1, const TQuat<T>& quat2, const TQuat<T>& tang2, T Alpha)
	{
		const TQuat<T> Q1 = TQuat::SlerpFullPath_NotNormalized(quat1, quat2, Alpha);
		const TQuat<T> Q2 = TQuat::SlerpFullPath_NotNormalized(tang1, tang2, Alpha);
		const TQuat<T> Result = TQuat::SlerpFullPath(Q1, Q2, 2.f * Alpha * (1.f - Alpha));

		return Result;
	}

	template<typename T>
	void TQuat<T>::CalcTangents(const TQuat<T>& PrevP, const TQuat<T>& P, const TQuat<T>& NextP, T Tension, TQuat<T>& OutTan)
	{
		const TQuat<T> InvP = P.Inverse();
		const TQuat<T> Part1 = (InvP * PrevP).Log();
		const TQuat<T> Part2 = (InvP * NextP).Log();

		const TQuat<T> PreExp = (Part1 + Part2) * -0.5f;

		OutTan = P * PreExp.Exp();
	}

	template<typename T>
	TVector<T> TQuat<T>::Euler() const
	{
		return Rotator().Euler();
	}

	template<typename T>
	bool TQuat<T>::NetSerialize(FArchive& Ar, class UPackageMap*, bool& bOutSuccess)
	{
		Ar.UsingCustomVersion(FEngineNetworkCustomVersion::Guid);

		TQuat<T> Q;

		if (Ar.IsSaving())
		{
			Q = *this;

			// Make sure we have a non null SquareSum. It shouldn't happen with a quaternion, but better be safe.
			if (Q.SizeSquared() <= (T)UE_SMALL_NUMBER)
			{
				Q = TQuat<T>::Identity;
			}
			else
			{
				// All transmitted quaternions *MUST BE* unit quaternions, in which case we can deduce the value of W.
				if (!ensure(Q.IsNormalized()))
				{
					Q.Normalize();
				}
				// force W component to be non-negative
				if (Q.W < 0.f)
				{
					Q.X *= -1.f;
					Q.Y *= -1.f;
					Q.Z *= -1.f;
					Q.W *= -1.f;
				}
			}
		}

		if (Ar.EngineNetVer() >= FEngineNetworkCustomVersion::SerializeDoubleVectorsAsDoubles && Ar.EngineNetVer() != FEngineNetworkCustomVersion::Ver21AndViewPitchOnly_DONOTUSE)
		{
			Ar << Q.X << Q.Y << Q.Z;
		}
		else
		{
			checkf(Ar.IsLoading(), TEXT("float -> double conversion applied outside of load!"));
			// Always serialize as float
			float SX, SY, SZ;
			Ar << SX << SY << SZ;
			Q.X = SX;
			Q.Y = SY;
			Q.Z = SZ;
		}

		if (Ar.IsLoading())
		{
			const T XYZMagSquared = (Q.X * Q.X + Q.Y * Q.Y + Q.Z * Q.Z);
			const T WSquared = (T)1.0f - XYZMagSquared;
			// If mag of (X,Y,Z) <= 1.0, then we calculate W to make magnitude of Q 1.0
			if (WSquared >= 0.f)
			{
				Q.W = FMath::Sqrt(WSquared);
			}
			// If mag of (X,Y,Z) > 1.0, we set W to zero, and then renormalize 
			else
			{
				Q.W = 0.f;

				const T XYZInvMag = FMath::InvSqrt(XYZMagSquared);
				Q.X *= XYZInvMag;
				Q.Y *= XYZInvMag;
				Q.Z *= XYZInvMag;
			}

			*this = Q;
		}

		bOutSuccess = true;
		return true;
	}


	//
	// Based on:
	// http://lolengine.net/blog/2014/02/24/quaternion-from-two-vectors-final
	// http://www.euclideanspace.com/maths/algebra/vectors/angleBetween/index.htm
	//
	template<typename T>
	FORCEINLINE_DEBUGGABLE TQuat<T> FindBetween_Helper(const TVector<T>& A, const TVector<T>& B, T NormAB)
	{
		T W = NormAB + TVector<T>::DotProduct(A, B);
		TQuat<T> Result;

		if (W >= 1e-6f * NormAB)
		{
			//Result = FVector::CrossProduct(A, B);
			Result = TQuat<T>(
				A.Y * B.Z - A.Z * B.Y,
				A.Z * B.X - A.X * B.Z,
				A.X * B.Y - A.Y * B.X,
				W);
		}
		else
		{
			// A and B point in opposite directions
			W = 0.f;
			const T X = FMath::Abs(A.X);
			const T Y = FMath::Abs(A.Y);
			const T Z = FMath::Abs(A.Z);

			// Find orthogonal basis. 
			const TVector<T> Basis = (X > Y && X > Z) ? TVector<T>::YAxisVector : -TVector<T>::XAxisVector;

			//Result = FVector::CrossProduct(A, Basis);
			Result = TQuat<T>(
				A.Y * Basis.Z - A.Z * Basis.Y,
				A.Z * Basis.X - A.X * Basis.Z,
				A.X * Basis.Y - A.Y * Basis.X,
				W);
		}

		Result.Normalize();
		return Result;
	}

	template<typename T>
	TQuat<T> TQuat<T>::FindBetweenNormals(const TVector<T>& A, const TVector<T>& B)
	{
		const T NormAB = 1.f;
		return UE::Math::FindBetween_Helper(A, B, NormAB);
	}

	template<typename T>
	TQuat<T> TQuat<T>::FindBetweenVectors(const TVector<T>& A, const TVector<T>& B)
	{
		const T NormAB = FMath::Sqrt(A.SizeSquared() * B.SizeSquared());
		return UE::Math::FindBetween_Helper(A, B, NormAB);
	}

	template<typename T>
	TQuat<T> TQuat<T>::Log() const
	{
		TQuat<T> Result;
		Result.W = 0.f;

		if (FMath::Abs(W) < 1.f)
		{
			const T Angle = FMath::Acos(W);
			const T SinAngle = FMath::Sin(Angle);

			if (FMath::Abs(SinAngle) >= T(UE_SMALL_NUMBER))
			{
				const T Scale = Angle / SinAngle;
				Result.X = Scale * X;
				Result.Y = Scale * Y;
				Result.Z = Scale * Z;

				return Result;
			}
		}

		Result.X = X;
		Result.Y = Y;
		Result.Z = Z;

		return Result;
	}

	template<typename T>
	TQuat<T> TQuat<T>::Exp() const
	{
		const T Angle = FMath::Sqrt(X * X + Y * Y + Z * Z);
		const T SinAngle = FMath::Sin(Angle);

		TQuat<T>  Result;
		Result.W = FMath::Cos(Angle);

		if (FMath::Abs(SinAngle) >= T(UE_SMALL_NUMBER))
		{
			const T Scale = SinAngle / Angle;
			Result.X = Scale * X;
			Result.Y = Scale * Y;
			Result.Z = Scale * Z;
		}
		else
		{
			Result.X = X;
			Result.Y = Y;
			Result.Z = Z;
		}

		return Result;
	}


} // namespace UE::Math
} // namespace UE

template<typename T>
static void FindBounds( T& OutMin, T& OutMax,  T Start, T StartLeaveTan, float StartT, T End, T EndArriveTan, float EndT, bool bCurve )
{
	OutMin = FMath::Min( Start, End );
	OutMax = FMath::Max( Start, End );

	// Do we need to consider extermeties of a curve?
	if(bCurve)
	{
		// Scale tangents based on time interval, so this code matches the behaviour in FInterpCurve::Eval
		T Diff = EndT - StartT;
		StartLeaveTan *= Diff;
		EndArriveTan *= Diff;

		const T a = 6.f*Start + 3.f*StartLeaveTan + 3.f*EndArriveTan - 6.f*End;
		const T b = -6.f*Start - 4.f*StartLeaveTan - 2.f*EndArriveTan + 6.f*End;
		const T c = StartLeaveTan;

		if (FMath::IsNearlyZero(a))
		{
			// The derivative is linear, find the linear root.
			if (!FMath::IsNearlyZero(b))
			{
				const T x = -c / b;
				const T t = StartT + x * (EndT - StartT);
				if (t > StartT && t < EndT)
				{
					const T Val = FMath::CubicInterp(Start, StartLeaveTan, End, EndArriveTan, x);
					OutMin = FMath::Min(OutMin, Val);
					OutMax = FMath::Max(OutMax, Val);
				}
			}
		}
		else
		{
			// The derivative is quadratic, find the quadratic roots.
			const T Discriminant = (b * b) - (4.f * a * c);
			if (Discriminant >= 0.f)
			{
				const T SqrtDisc = FMath::Sqrt(Discriminant);

				const T x0 = (-b + SqrtDisc) / (2.f * a); // x0 is the 'Alpha' ie between 0 and 1
				const T t0 = StartT + x0 * (EndT - StartT); // Then t0 is the actual 'time' on the curve
				if (t0 > StartT && t0 < EndT)
				{
					const T Val = FMath::CubicInterp(Start, StartLeaveTan, End, EndArriveTan, x0);

					OutMin = FMath::Min(OutMin, Val);
					OutMax = FMath::Max(OutMax, Val);
				}

				const T x1 = (-b - SqrtDisc) / (2.f * a);
				const T t1 = StartT + x1 * (EndT - StartT);
				if (t1 > StartT && t1 < EndT)
				{
					const T Val = FMath::CubicInterp(Start, StartLeaveTan, End, EndArriveTan, x1);

					OutMin = FMath::Min(OutMin, Val);
					OutMax = FMath::Max(OutMax, Val);
				}
			}
		}
	}
}

void CORE_API CurveFloatFindIntervalBounds( const FInterpCurvePoint<float>& Start, const FInterpCurvePoint<float>& End, float& CurrentMin, float& CurrentMax )
{
	const bool bIsCurve = Start.IsCurveKey();

	float OutMin, OutMax;

	FindBounds(OutMin, OutMax, Start.OutVal, Start.LeaveTangent, Start.InVal, End.OutVal, End.ArriveTangent, End.InVal, bIsCurve);

	CurrentMin = FMath::Min( CurrentMin, OutMin );
	CurrentMax = FMath::Max( CurrentMax, OutMax );
}

void CORE_API CurveVector2DFindIntervalBounds( const FInterpCurvePoint<FVector2D>& Start, const FInterpCurvePoint<FVector2D>& End, FVector2D& CurrentMin, FVector2D& CurrentMax )
{
	const bool bIsCurve = Start.IsCurveKey();

	FVector2D::FReal OutMin, OutMax;

	FindBounds(OutMin, OutMax, Start.OutVal.X, Start.LeaveTangent.X, Start.InVal, End.OutVal.X, End.ArriveTangent.X, End.InVal, bIsCurve);
	CurrentMin.X = FMath::Min( CurrentMin.X, OutMin );
	CurrentMax.X = FMath::Max( CurrentMax.X, OutMax );

	FindBounds(OutMin, OutMax, Start.OutVal.Y, Start.LeaveTangent.Y, Start.InVal, End.OutVal.Y, End.ArriveTangent.Y, End.InVal, bIsCurve);
	CurrentMin.Y = FMath::Min( CurrentMin.Y, OutMin );
	CurrentMax.Y = FMath::Max( CurrentMax.Y, OutMax );
}

void CORE_API CurveVectorFindIntervalBounds( const FInterpCurvePoint<FVector>& Start, const FInterpCurvePoint<FVector>& End, FVector& CurrentMin, FVector& CurrentMax )
{
	const bool bIsCurve = Start.IsCurveKey();

	FVector::FReal OutMin, OutMax;

	FindBounds(OutMin, OutMax, Start.OutVal.X, Start.LeaveTangent.X, Start.InVal, End.OutVal.X, End.ArriveTangent.X, End.InVal, bIsCurve);
	CurrentMin.X = FMath::Min( CurrentMin.X, OutMin );
	CurrentMax.X = FMath::Max( CurrentMax.X, OutMax );

	FindBounds(OutMin, OutMax, Start.OutVal.Y, Start.LeaveTangent.Y, Start.InVal, End.OutVal.Y, End.ArriveTangent.Y, End.InVal, bIsCurve);
	CurrentMin.Y = FMath::Min( CurrentMin.Y, OutMin );
	CurrentMax.Y = FMath::Max( CurrentMax.Y, OutMax );

	FindBounds(OutMin, OutMax, Start.OutVal.Z, Start.LeaveTangent.Z, Start.InVal, End.OutVal.Z, End.ArriveTangent.Z, End.InVal, bIsCurve);
	CurrentMin.Z = FMath::Min( CurrentMin.Z, OutMin );
	CurrentMax.Z = FMath::Max( CurrentMax.Z, OutMax );
}

void CORE_API CurveTwoVectorsFindIntervalBounds(const FInterpCurvePoint<FTwoVectors>& Start, const FInterpCurvePoint<FTwoVectors>& End, FTwoVectors& CurrentMin, FTwoVectors& CurrentMax)
{
	const bool bIsCurve = Start.IsCurveKey();

	FVector::FReal OutMin, OutMax;

	// Do the first curve
	FindBounds(OutMin, OutMax, Start.OutVal.v1.X, Start.LeaveTangent.v1.X, Start.InVal, End.OutVal.v1.X, End.ArriveTangent.v1.X, End.InVal, bIsCurve);
	CurrentMin.v1.X = FMath::Min( CurrentMin.v1.X, OutMin );
	CurrentMax.v1.X = FMath::Max( CurrentMax.v1.X, OutMax );

	FindBounds(OutMin, OutMax, Start.OutVal.v1.Y, Start.LeaveTangent.v1.Y, Start.InVal, End.OutVal.v1.Y, End.ArriveTangent.v1.Y, End.InVal, bIsCurve);
	CurrentMin.v1.Y = FMath::Min( CurrentMin.v1.Y, OutMin );
	CurrentMax.v1.Y = FMath::Max( CurrentMax.v1.Y, OutMax );

	FindBounds(OutMin, OutMax, Start.OutVal.v1.Z, Start.LeaveTangent.v1.Z, Start.InVal, End.OutVal.v1.Z, End.ArriveTangent.v1.Z, End.InVal, bIsCurve);
	CurrentMin.v1.Z = FMath::Min( CurrentMin.v1.Z, OutMin );
	CurrentMax.v1.Z = FMath::Max( CurrentMax.v1.Z, OutMax );

	// Do the second curve
	FindBounds(OutMin, OutMax, Start.OutVal.v2.X, Start.LeaveTangent.v2.X, Start.InVal, End.OutVal.v2.X, End.ArriveTangent.v2.X, End.InVal, bIsCurve);
	CurrentMin.v2.X = FMath::Min( CurrentMin.v2.X, OutMin );
	CurrentMax.v2.X = FMath::Max( CurrentMax.v2.X, OutMax );

	FindBounds(OutMin, OutMax, Start.OutVal.v2.Y, Start.LeaveTangent.v2.Y, Start.InVal, End.OutVal.v2.Y, End.ArriveTangent.v2.Y, End.InVal, bIsCurve);
	CurrentMin.v2.Y = FMath::Min( CurrentMin.v2.Y, OutMin );
	CurrentMax.v2.Y = FMath::Max( CurrentMax.v2.Y, OutMax );

	FindBounds(OutMin, OutMax, Start.OutVal.v2.Z, Start.LeaveTangent.v2.Z, Start.InVal, End.OutVal.v2.Z, End.ArriveTangent.v2.Z, End.InVal, bIsCurve);
	CurrentMin.v2.Z = FMath::Min( CurrentMin.v2.Z, OutMin );
	CurrentMax.v2.Z = FMath::Max( CurrentMax.v2.Z, OutMax );
}

void CORE_API CurveLinearColorFindIntervalBounds( const FInterpCurvePoint<FLinearColor>& Start, const FInterpCurvePoint<FLinearColor>& End, FLinearColor& CurrentMin, FLinearColor& CurrentMax )
{
	const bool bIsCurve = Start.IsCurveKey();

	float OutMin, OutMax;

	FindBounds(OutMin, OutMax, Start.OutVal.R, Start.LeaveTangent.R, Start.InVal, End.OutVal.R, End.ArriveTangent.R, End.InVal, bIsCurve);
	CurrentMin.R = FMath::Min( CurrentMin.R, OutMin );
	CurrentMax.R = FMath::Max( CurrentMax.R, OutMax );

	FindBounds(OutMin, OutMax, Start.OutVal.G, Start.LeaveTangent.G, Start.InVal, End.OutVal.G, End.ArriveTangent.G, End.InVal, bIsCurve);
	CurrentMin.G = FMath::Min( CurrentMin.G, OutMin );
	CurrentMax.G = FMath::Max( CurrentMax.G, OutMax );

	FindBounds(OutMin, OutMax, Start.OutVal.B, Start.LeaveTangent.B, Start.InVal, End.OutVal.B, End.ArriveTangent.B, End.InVal, bIsCurve);
	CurrentMin.B = FMath::Min( CurrentMin.B, OutMin );
	CurrentMax.B = FMath::Max( CurrentMax.B, OutMax );

	FindBounds(OutMin, OutMax, Start.OutVal.A, Start.LeaveTangent.A, Start.InVal, End.OutVal.A, End.ArriveTangent.A, End.InVal, bIsCurve);
	CurrentMin.A = FMath::Min( CurrentMin.A, OutMin );
	CurrentMax.A = FMath::Max( CurrentMax.A, OutMax );
}

CORE_API float FMath::PointDistToLine(const FVector &Point, const FVector &Direction, const FVector &Origin, FVector &OutClosestPoint)
{
	const FVector SafeDir = Direction.GetSafeNormal();
	OutClosestPoint = Origin + (SafeDir * ((Point-Origin) | SafeDir));
	return (float)(OutClosestPoint-Point).Size();		// LWC_TODO: Precision loss
}

CORE_API float FMath::PointDistToLine(const FVector &Point, const FVector &Direction, const FVector &Origin)
{
	const FVector SafeDir = Direction.GetSafeNormal();
	const FVector OutClosestPoint = Origin + (SafeDir * ((Point-Origin) | SafeDir));
	return (float)(OutClosestPoint-Point).Size();		// LWC_TODO: Precision loss
}

template <typename T>
UE::Math::TVector<T> FMath::ClosestPointOnSegment(const UE::Math::TVector<T>&Point, const UE::Math::TVector<T>&StartPoint, const UE::Math::TVector<T>&EndPoint)
{
	using VectorType = UE::Math::TVector<T>;

	const VectorType Segment = EndPoint - StartPoint;
	const VectorType VectToPoint = Point - StartPoint;

	// See if closest point is before StartPoint
	const T Dot1 = VectToPoint | Segment;
	if( Dot1 <= 0 )
	{
		return StartPoint;
	}

	// See if closest point is beyond EndPoint
	const T Dot2 = Segment | Segment;
	if( Dot2 <= Dot1 )
	{
		return EndPoint;
	}

	// Closest Point is within segment
	return StartPoint + Segment * (Dot1 / Dot2);
}

template <typename T>
UE::Math::TVector2<T> FMath::ClosestPointOnSegment2D(const UE::Math::TVector2<T>& Point, const UE::Math::TVector2<T>& StartPoint, const UE::Math::TVector2<T>& EndPoint)
{
	using VectorType = UE::Math::TVector2<T>;

	const VectorType Segment = EndPoint - StartPoint;
	const VectorType VectToPoint = Point - StartPoint;

	// See if closest point is before StartPoint
	const T Dot1 = VectToPoint | Segment;
	if (Dot1 <= 0)
	{
		return StartPoint;
	}

	// See if closest point is beyond EndPoint
	const T Dot2 = Segment | Segment;
	if (Dot2 <= Dot1)
	{
		return EndPoint;
	}

	// Closest Point is within segment
	return StartPoint + Segment * (Dot1 / Dot2);
}

float FMath::PointDistToSegment(const FVector &Point, const FVector &StartPoint, const FVector &EndPoint)
{
	const FVector ClosestPoint = ClosestPointOnSegment(Point, StartPoint, EndPoint);
	return float((Point - ClosestPoint).Size());	// LWC_TODO: Precision loss
}

float FMath::PointDistToSegmentSquared(const FVector &Point, const FVector &StartPoint, const FVector &EndPoint)
{
	const FVector ClosestPoint = ClosestPointOnSegment(Point, StartPoint, EndPoint);
	return float((Point - ClosestPoint).SizeSquared());	// LWC_TODO: Precision loss
}

template<typename T>
struct SegmentDistToSegment_Solver
{
	using VectorType = UE::Math::TVector<T>;

	SegmentDistToSegment_Solver(const VectorType& InA1, const VectorType& InB1, const VectorType& InA2, const VectorType& InB2):
		bLinesAreNearlyParallel(false),
		A1(InA1),
		A2(InA2),
		S1(InB1 - InA1),
		S2(InB2 - InA2),
		S3(InA1 - InA2)
	{
	}

	bool bLinesAreNearlyParallel;

	const VectorType& A1;
	const VectorType& A2;
	const VectorType S1;
	const VectorType S2;
	const VectorType S3;

	void Solve(VectorType& OutP1, VectorType& OutP2)
	{
		const T Dot11 = S1 | S1;
		const T Dot12 = S1 | S2;
		const T Dot13 = S1 | S3;
		const T Dot22 = S2 | S2;
		const T Dot23 = S2 | S3;

		const T D = Dot11*Dot22 - Dot12*Dot12;

		T D1 = D;
		T D2 = D;

		T N1;
		T N2;

		if (bLinesAreNearlyParallel || D < UE_KINDA_SMALL_NUMBER)
		{
			// the lines are almost parallel
			N1 = 0.f;	// force using point A on segment S1
			D1 = 1.f;	// to prevent possible division by 0 later
			N2 = Dot23;
			D2 = Dot22;
		}
		else
		{
			// get the closest points on the infinite lines
			N1 = (Dot12*Dot23 - Dot22*Dot13);
			N2 = (Dot11*Dot23 - Dot12*Dot13);

			if (N1 < 0.f)
			{
				// t1 < 0.f => the s==0 edge is visible
				N1 = 0.f;
				N2 = Dot23;
				D2 = Dot22;
			}
			else if (N1 > D1)
			{
				// t1 > 1 => the t1==1 edge is visible
				N1 = D1;
				N2 = Dot23 + Dot12;
				D2 = Dot22;
			}
		}

		if (N2 < 0.f)
		{
			// t2 < 0 => the t2==0 edge is visible
			N2 = 0.f;

			// recompute t1 for this edge
			if (-Dot13 < 0.f)
			{
				N1 = 0.f;
			}
			else if (-Dot13 > Dot11)
			{
				N1 = D1;
			}
			else
			{
				N1 = -Dot13;
				D1 = Dot11;
			}
		}
		else if (N2 > D2)
		{
			// t2 > 1 => the t2=1 edge is visible
			N2 = D2;

			// recompute t1 for this edge
			if ((-Dot13 + Dot12) < 0.f)
			{
				N1 = 0.f;
			}
			else if ((-Dot13 + Dot12) > Dot11)
			{
				N1 = D1;
			}
			else
			{
				N1 = (-Dot13 + Dot12);
				D1 = Dot11;
			}
		}

		// finally do the division to get the points' location
		const T T1 = (FMath::Abs(N1) < UE_KINDA_SMALL_NUMBER ? 0.f : N1 / D1);
		const T T2 = (FMath::Abs(N2) < UE_KINDA_SMALL_NUMBER ? 0.f : N2 / D2);

		// return the closest points
		OutP1 = A1 + T1 * S1;
		OutP2 = A2 + T2 * S2;
	}
};

template<typename T>
void FMath::SegmentDistToSegmentSafe(UE::Math::TVector<T> A1, UE::Math::TVector<T> B1, UE::Math::TVector<T> A2, UE::Math::TVector<T> B2, UE::Math::TVector<T>& OutP1, UE::Math::TVector<T>& OutP2)
{
	using VectorType = UE::Math::TVector<T>;

	SegmentDistToSegment_Solver<T> Solver(A1, B1, A2, B2);

	const VectorType S1_norm = Solver.S1.GetSafeNormal();
	const VectorType S2_norm = Solver.S2.GetSafeNormal();

	const bool bS1IsPoint = S1_norm.IsZero();
	const bool bS2IsPoint = S2_norm.IsZero();

	if (bS1IsPoint && bS2IsPoint)
	{
		OutP1 = A1;
		OutP2 = A2;
	}
	else if (bS2IsPoint)
	{
		OutP1 = ClosestPointOnSegment(A2, A1, B1);
		OutP2 = A2;
	}
	else if (bS1IsPoint)
	{
		OutP1 = A1;
		OutP2 = ClosestPointOnSegment(A1, A2, B2);
	}
	else
	{
		const T Dot11_norm = S1_norm | S1_norm;	// always >= 0
		const T Dot22_norm = S2_norm | S2_norm;	// always >= 0
		const T Dot12_norm = S1_norm | S2_norm;
		const T D_norm = Dot11_norm * Dot22_norm - Dot12_norm * Dot12_norm;	// always >= 0

		Solver.bLinesAreNearlyParallel = D_norm < UE_KINDA_SMALL_NUMBER;
		Solver.Solve(OutP1, OutP2);
	}
}

void FMath::SegmentDistToSegment(FVector A1, FVector B1, FVector A2, FVector B2, FVector& OutP1, FVector& OutP2)
{
	SegmentDistToSegment_Solver(A1, B1, A2, B2).Solve(OutP1, OutP2);
}

float FMath::GetTForSegmentPlaneIntersect(const FVector& StartPoint, const FVector& EndPoint, const FPlane& Plane)
{
	return float(( Plane.W - (StartPoint|Plane) ) / ( (EndPoint - StartPoint)|Plane));		// LWC_TODO: Precision loss
}

bool FMath::SegmentPlaneIntersection(const FVector& StartPoint, const FVector& EndPoint, const FPlane& Plane, FVector& out_IntersectionPoint)
{
	FVector::FReal T = FMath::GetTForSegmentPlaneIntersect(StartPoint, EndPoint, Plane);
	// If the parameter value is not between 0 and 1, there is no intersection
	if (T > -UE_KINDA_SMALL_NUMBER && T < 1.f + UE_KINDA_SMALL_NUMBER)
	{
		out_IntersectionPoint = StartPoint + T * (EndPoint - StartPoint);
		return true;
	}
	return false;
}

bool FMath::SegmentTriangleIntersection(const FVector& StartPoint, const FVector& EndPoint, const FVector& A, const FVector& B, const FVector& C, FVector& OutIntersectPoint, FVector& OutTriangleNormal)
{
	FVector Edge1(B - A);
	Edge1.Normalize();
	FVector Edge2(C - A);
	Edge2.Normalize();
	FVector TriNormal = Edge2 ^ Edge1;
	if (!TriNormal.Normalize())
	{
		return false;
	}

	bool bCollide = FMath::SegmentPlaneIntersection(StartPoint, EndPoint, FPlane(A, TriNormal), OutIntersectPoint);
	if (!bCollide)
	{
		return false;
	}

	// ComputeBarycentricTri returns false when the triangle is too small.
	FVector BaryCentric;
	if (!FMath::ComputeBarycentricTri(OutIntersectPoint, A, B, C, BaryCentric))
	{
		return false;
	}

	if (BaryCentric.X >= 0.0f && BaryCentric.Y >= 0.0f && BaryCentric.Z >= 0.0f)
	{
		OutTriangleNormal = TriNormal;
		return true;
	}
	return false;
}

bool FMath::SegmentIntersection2D(const FVector& SegmentStartA, const FVector& SegmentEndA, const FVector& SegmentStartB, const FVector& SegmentEndB, FVector& out_IntersectionPoint)
{
	const FVector VectorA = SegmentEndA - SegmentStartA;
	const FVector VectorB = SegmentEndB - SegmentStartB;

	const FVector::FReal S = (-VectorA.Y * (SegmentStartA.X - SegmentStartB.X) + VectorA.X * (SegmentStartA.Y - SegmentStartB.Y)) / (-VectorB.X * VectorA.Y + VectorA.X * VectorB.Y);
	const FVector::FReal T = (VectorB.X * (SegmentStartA.Y - SegmentStartB.Y) - VectorB.Y * (SegmentStartA.X - SegmentStartB.X)) / (-VectorB.X * VectorA.Y + VectorA.X * VectorB.Y);

	const bool bIntersects = (S >= 0 && S <= 1 && T >= 0 && T <= 1);

	if (bIntersects)
	{
		out_IntersectionPoint.X = SegmentStartA.X + (T * VectorA.X);
		out_IntersectionPoint.Y = SegmentStartA.Y + (T * VectorA.Y);
		out_IntersectionPoint.Z = SegmentStartA.Z + (T * VectorA.Z);
	}

	return bIntersects;
}

/**
 * Compute the screen bounds of a point light along one axis.
 * Based on http://www.gamasutra.com/features/20021011/lengyel_06.htm
 * and http://sourceforge.net/mailarchive/message.php?msg_id=10501105
 */
static bool ComputeProjectedSphereShaft(
	FVector::FReal LightX,
	FVector::FReal LightZ,
	FVector::FReal Radius,
	const FMatrix& ProjMatrix,
	const FVector& Axis,
	FVector::FReal AxisSign,
	int32& InOutMinX,
	int32& InOutMaxX
	)
{
	FVector::FReal ViewX = (FVector::FReal)InOutMinX;
	FVector::FReal ViewSizeX = (FVector::FReal)(InOutMaxX - InOutMinX);

	// Vertical planes: T = <Nx, 0, Nz, 0>
	FVector::FReal Discriminant = (FMath::Square(LightX) - FMath::Square(Radius) + FMath::Square(LightZ)) * FMath::Square(LightZ);
	if(Discriminant >= 0)
	{
		FVector::FReal SqrtDiscriminant = FMath::Sqrt(Discriminant);
		FVector::FReal InvLightSquare = 1.0f / (FMath::Square(LightX) + FMath::Square(LightZ));

		FVector::FReal Nxa = (Radius * LightX - SqrtDiscriminant) * InvLightSquare;
		FVector::FReal Nxb = (Radius * LightX + SqrtDiscriminant) * InvLightSquare;
		FVector::FReal Nza = (Radius - Nxa * LightX) / LightZ;
		FVector::FReal Nzb = (Radius - Nxb * LightX) / LightZ;
		FVector::FReal Pza = LightZ - Radius * Nza;
		FVector::FReal Pzb = LightZ - Radius * Nzb;

		using FVec4Real = decltype(FVector4::X);

		// Tangent a
		if(Pza > 0)
		{
			FVector::FReal Pxa = -Pza * Nza / Nxa;
			FVector4 P = ProjMatrix.TransformFVector4(FVector4(FVec4Real(Axis.X * Pxa),FVec4Real(Axis.Y * Pxa),FVec4Real(Pza),1));
			FVector::FReal X = (Dot3(P,Axis) / P.W + 1.0f * AxisSign) / 2.0f * AxisSign;
			if((Nxa < FVector::FReal{0}) ^ (AxisSign < FVector::FReal{0}))
			{
				InOutMaxX = FMath::Min(FMath::CeilToInt32(ViewSizeX * X + ViewX),InOutMaxX);
			}
			else
			{
				InOutMinX = FMath::Max(FMath::FloorToInt32(ViewSizeX * X + ViewX),InOutMinX);
			}
		}

		// Tangent b
		if(Pzb > 0)
		{
			FVector::FReal Pxb = -Pzb * Nzb / Nxb;
			FVector4 P = ProjMatrix.TransformFVector4(FVector4(FVec4Real(Axis.X * Pxb), FVec4Real(Axis.Y * Pxb),FVec4Real(Pzb),1));
			FVector::FReal X = (Dot3(P,Axis) / P.W + 1.0f * AxisSign) / 2.0f * AxisSign;
			if((Nxb < FVector::FReal{0}) ^ (AxisSign < FVector::FReal{0}))
			{
				InOutMaxX = FMath::Min(FMath::CeilToInt32(ViewSizeX * X + ViewX),InOutMaxX);
			}
			else
			{
				InOutMinX = FMath::Max(FMath::FloorToInt32(ViewSizeX * X + ViewX),InOutMinX);
			}
		}
	}

	return InOutMinX < InOutMaxX;
}

uint32 FMath::ComputeProjectedSphereScissorRect(FIntRect& InOutScissorRect, FVector SphereOrigin, float Radius, FVector ViewOrigin, const FMatrix& ViewMatrix, const FMatrix& ProjMatrix)
{
	// Calculate a scissor rectangle for the light's radius.
	if((SphereOrigin - ViewOrigin).SizeSquared() > FMath::Square(Radius))
	{
		FVector LightVector = ViewMatrix.TransformPosition(SphereOrigin);

		if(!ComputeProjectedSphereShaft(
			LightVector.X,
			LightVector.Z,
			Radius,
			ProjMatrix,
			FVector(+1,0,0),
			+1,
			InOutScissorRect.Min.X,
			InOutScissorRect.Max.X))
		{
			return 0;
		}

		if(!ComputeProjectedSphereShaft(
			LightVector.Y,
			LightVector.Z,
			Radius,
			ProjMatrix,
			FVector(0,+1,0),
			-1,
			InOutScissorRect.Min.Y,
			InOutScissorRect.Max.Y))
		{
			return 0;
		}

		return 1;
	}
	else
	{
		return 2;
	}
}

int32 FMath::PlaneAABBRelativePosition(const FPlane& P, const FBox& AABB)
{
	// find diagonal most closely aligned with normal of plane
	FVector Vmin, Vmax;

	// Bypass the slow FVector[] operator. Not RESTRICT because it won't update Vmin, Vmax
	FVector::FReal* VminPtr = (FVector::FReal*)&Vmin;
	FVector::FReal* VmaxPtr = (FVector::FReal*)&Vmax;

	// Use restrict to get better instruction scheduling and to bypass the slow FVector[] operator
	const FVector::FReal* RESTRICT AABBMinPtr = (const FVector::FReal*)&AABB.Min;
	const FVector::FReal* RESTRICT AABBMaxPtr = (const FVector::FReal*)&AABB.Max;
	const FPlane::FReal* RESTRICT PlanePtr = (const FPlane::FReal*)&P;

	for(int32 Idx=0;Idx<3;++Idx)
	{
		if(PlanePtr[Idx] >= 0.f)
		{
			VminPtr[Idx] = AABBMinPtr[Idx];
			VmaxPtr[Idx] = AABBMaxPtr[Idx];
		}
		else
		{
			VminPtr[Idx] = AABBMaxPtr[Idx];
			VmaxPtr[Idx] = AABBMinPtr[Idx]; 
		}
	}

	// if either diagonal is right on the plane, or one is on either side we have an interesection
	FPlane::FReal dMax = P.PlaneDot(Vmax);
	FPlane::FReal dMin = P.PlaneDot(Vmin);

	// if Max is below plane, or Min is above we know there is no intersection.. otherwise there must be one
	if (dMax < 0.f)
	{
		return -1;
	}
	else if (dMin > 0.f)
	{
		return 1;
	}
	return 0;
}

bool FMath::PlaneAABBIntersection(const FPlane& P, const FBox& AABB)
{
	return PlaneAABBRelativePosition(P, AABB) == 0;
}

bool FMath::SphereConeIntersection(const FVector& SphereCenter, float SphereRadius, const FVector& ConeAxis, float ConeAngleSin, float ConeAngleCos)
{
	/**
	 * from http://www.geometrictools.com/Documentation/IntersectionSphereCone.pdf
	 * (Copyright c 1998-2008. All Rights Reserved.) http://www.geometrictools.com (boost license)
	 */

	// the following code assumes the cone tip is at 0,0,0 (means the SphereCenter is relative to the cone tip)

	FVector U = ConeAxis * (-SphereRadius / ConeAngleSin);
	FVector D = SphereCenter - U;
	FVector::FReal dsqr = D | D;
	FVector::FReal e = ConeAxis | D;

	if(e > 0 && e * e >= dsqr * FMath::Square(ConeAngleCos))
	{
		dsqr = SphereCenter |SphereCenter;
		e = -ConeAxis | SphereCenter;
		if(e > 0 && e*e >= dsqr * FMath::Square(ConeAngleSin))
		{
			return dsqr <= FMath::Square(SphereRadius);
		}
		else
		{
			return true;
		}
	}
	return false;
}

FVector FMath::ClosestPointOnTriangleToPoint(const FVector& Point, const FVector& A, const FVector& B, const FVector& C)
{
	//Figure out what region the point is in and compare against that "point" or "edge"
	const FVector BA = A - B;
	const FVector AC = C - A;
	const FVector CB = B - C;
	const FVector TriNormal = BA ^ CB;

	// Get the planes that define this triangle
	// edges BA, AC, BC with normals perpendicular to the edges facing outward
	const FPlane Planes[3] = { FPlane(B, TriNormal ^ BA), FPlane(A, TriNormal ^ AC), FPlane(C, TriNormal ^ CB) };
	int32 PlaneHalfspaceBitmask = 0;

	//Determine which side of each plane the test point exists
	for (int32 i=0; i<3; i++)
	{
		if (Planes[i].PlaneDot(Point) > 0.0f)
		{
			PlaneHalfspaceBitmask |= (1 << i);
		}
	}

	FVector Result(Point.X, Point.Y, Point.Z);
	switch (PlaneHalfspaceBitmask)
	{
	case 0: //000 Inside
		return FVector::PointPlaneProject(Point, A, B, C);
	case 1:	//001 Segment BA
		Result = FMath::ClosestPointOnSegment(Point, B, A);
		break;
	case 2:	//010 Segment AC
		Result = FMath::ClosestPointOnSegment(Point, A, C);
		break;
	case 3:	//011 point A
		return A;
	case 4: //100 Segment BC
		Result = FMath::ClosestPointOnSegment(Point, B, C);
		break;
	case 5: //101 point B
		return B;
	case 6: //110 point C
		return C;
	default:
		UE_LOGF(LogUnrealMath, Log, "Impossible result in FMath::ClosestPointOnTriangleToPoint");
		break;
	}

	return Result;
}

FVector FMath::GetBaryCentric2D(const FVector& Point, const FVector& A, const FVector& B, const FVector& C)
{
	FVector::FReal a = ((B.Y-C.Y)*(Point.X-C.X) + (C.X-B.X)*(Point.Y-C.Y)) / ((B.Y-C.Y)*(A.X-C.X) + (C.X-B.X)*(A.Y-C.Y));
	FVector::FReal b = ((C.Y-A.Y)*(Point.X-C.X) + (A.X-C.X)*(Point.Y-C.Y)) / ((B.Y-C.Y)*(A.X-C.X) + (C.X-B.X)*(A.Y-C.Y));

	return FVector(a, b, 1.0f - a - b);	
}

FVector FMath::GetBaryCentric2D(const FVector2D& Point, const FVector2D& A, const FVector2D& B, const FVector2D& C)
{
	FVector2D::FReal a = ((B.Y - C.Y) * (Point.X - C.X) + (C.X - B.X) * (Point.Y - C.Y)) / ((B.Y - C.Y) * (A.X - C.X) + (C.X - B.X) * (A.Y - C.Y));
	FVector2D::FReal b = ((C.Y - A.Y) * (Point.X - C.X) + (A.X - C.X) * (Point.Y - C.Y)) / ((B.Y - C.Y) * (A.X - C.X) + (C.X - B.X) * (A.Y - C.Y));

	return FVector(a, b, 1.0f - a - b);
}

bool FMath::ComputeBarycentricTri(const FVector& Point, const FVector& A, const FVector& B, const FVector& C, FVector& OutBarycentric, double Tolerance)
{
	// Tolerance cannot be negative
	checkSlow(Tolerance >= 0);

	// Compute the normal direction of the triangle (scaled by 2x area)
	const FVector TriNorm = (B-A) ^ (C-A);

	// Check if the triangle is too small, according to the tolerance
	const FVector::FReal TriNormSizeSquared = TriNorm.SizeSquared();
	if(TriNormSizeSquared <= Tolerance)
	{
		return false;
	}

	// Compute 1 / twice area of triangle ABC
	const FVector::FReal AreaABCInv = FMath::InvSqrt(TriNormSizeSquared);

	// Compute unit-length Tri normal
	const FVector N = TriNorm * AreaABCInv;

	// Compute a contribution (in X)
	const FVector::FReal AreaPBC = N | ((B-Point) ^ (C-Point));
	OutBarycentric.X = AreaPBC * AreaABCInv;

	// Compute b contribution (in Y)
	const FVector::FReal AreaPCA = N | ((C-Point) ^ (A-Point));
	OutBarycentric.Y = AreaPCA * AreaABCInv;

	// Compute c contribution (in Z)
	OutBarycentric.Z = 1.0 - OutBarycentric.X - OutBarycentric.Y;
	return true;
}

FVector FMath::ComputeBaryCentric2D(const FVector& Point, const FVector& A, const FVector& B, const FVector& C)
{
	FVector ToRet(0, 0, 0);
	if (!ComputeBarycentricTri(Point, A, B, C, ToRet, UE_DOUBLE_SMALL_NUMBER))
	{
		UE_LOGF(LogUnrealMath, Warning, "Small triangle detected in FMath::ComputeBaryCentric2D(); can't compute valid barycentric coordinate.");
	}
	return ToRet;
}

FVector4 FMath::ComputeBaryCentric3D(const FVector& Point, const FVector& A, const FVector& B, const FVector& C, const FVector& D)
{	
	//http://www.devmaster.net/wiki/Barycentric_coordinates
	//Pick A as our origin and
	//Setup three basis vectors AB, AC, AD
	const FVector B1 = (B-A);
	const FVector B2 = (C-A);
	const FVector B3 = (D-A);

	//check co-planarity of A,B,C,D
	check( FMath::Abs(B1 | (B2 ^ B3)) > UE_SMALL_NUMBER && "Coplanar points in FMath::ComputeBaryCentric3D()");

	//Transform Point into this new space
	const FVector V = (Point - A);

	//Create a matrix of linearly independent vectors
	const FMatrix SolvMat(B1, B2, B3, FVector::ZeroVector);

	//The point V can be expressed as Ax=v where x is the vector containing the weights {w1...wn}
	//Solve for x by multiplying both sides by AInv   (AInv * A)x = AInv * v ==> x = AInv * v
	const FMatrix InvSolvMat = SolvMat.Inverse();
	const FPlane BaryCoords = (FPlane)InvSolvMat.TransformVector(V);

	//Reorder the weights to be a, b, c, d
	return FVector4(1.0f - BaryCoords.X - BaryCoords.Y - BaryCoords.Z, BaryCoords.X, BaryCoords.Y, BaryCoords.Z);
}

FVector FMath::ClosestPointOnTetrahedronToPoint(const FVector& Point, const FVector& A, const FVector& B, const FVector& C, const FVector& D)
{
	//Check for coplanarity of all four points
	check(FMath::Abs((C-A) | ((B-A)^(D-C))) > 0.0001f && "Coplanar points in FMath::ComputeBaryCentric3D()");

	//http://osdir.com/ml/games.devel.algorithms/2003-02/msg00394.html
	//     D
	//    /|\		  C-----------B
	//   / | \		   \         /
	//  /  |  \	   or	\  \A/	/
    // C   |   B		 \	|  /
	//  \  |  /			  \	| /
    //   \ | /			   \|/
	//     A				D
	
	// Figure out the ordering (is D in the direction of the CCW triangle ABC)
	FVector Pt1(A),Pt2(B),Pt3(C),Pt4(D);
 	const FPlane ABC(A,B,C);
 	if (ABC.PlaneDot(D) < 0.0f)
 	{
 		//Swap two points to maintain CCW orders
 		Pt3 = D;
 		Pt4 = C;
 	}
		
	//Tetrahedron made up of 4 CCW faces - DCA, DBC, DAB, ACB
	const FPlane Planes[4] = {FPlane(Pt4,Pt3,Pt1), FPlane(Pt4,Pt2,Pt3), FPlane(Pt4,Pt1,Pt2), FPlane(Pt1,Pt3,Pt2)};

	//Determine which side of each plane the test point exists
	int32 PlaneHalfspaceBitmask = 0;
	for (int32 i=0; i<4; i++)
	{
		if (Planes[i].PlaneDot(Point) > 0.0f)
		{
			PlaneHalfspaceBitmask |= (1 << i);
		}
	}

	//Verts + Faces - Edges = 2	(Euler)
	FVector Result(Point.X, Point.Y, Point.Z);
	switch (PlaneHalfspaceBitmask)
	{
	case 0:	 //inside (0000)
		//@TODO - could project point onto any face
		break;
	case 1:	 //0001 Face	DCA
		return FMath::ClosestPointOnTriangleToPoint(Point, Pt4, Pt3, Pt1);
	case 2:	 //0010 Face	DBC
		return FMath::ClosestPointOnTriangleToPoint(Point, Pt4, Pt2, Pt3);
	case 3:  //0011	Edge	DC
		Result = FMath::ClosestPointOnSegment(Point, Pt4, Pt3);
		break;
	case 4:	 //0100 Face	DAB
		return FMath::ClosestPointOnTriangleToPoint(Point, Pt4, Pt1, Pt2);
	case 5:  //0101	Edge	DA
		Result = FMath::ClosestPointOnSegment(Point, Pt4, Pt1);
		break;
	case 6:  //0110	Edge	DB
		Result = FMath::ClosestPointOnSegment(Point, Pt4, Pt2);
		break;
	case 7:	 //0111 Point	D
		return Pt4;
	case 8:	 //1000 Face	ACB
		return FMath::ClosestPointOnTriangleToPoint(Point, Pt1, Pt3, Pt2);
	case 9:  //1001	Edge	AC	
		Result = FMath::ClosestPointOnSegment(Point, Pt1, Pt3);
		break;
	case 10: //1010	Edge	BC
		Result = FMath::ClosestPointOnSegment(Point, Pt2, Pt3);
		break;
	case 11: //1011 Point	C
		return Pt3;
	case 12: //1100	Edge	BA
		Result = FMath::ClosestPointOnSegment(Point, Pt2, Pt1);
		break;
	case 13: //1101 Point	A
		return Pt1;
	case 14: //1110 Point	B
		return Pt2;
	default: //impossible (1111)
		UE_LOGF(LogUnrealMath, Log, "FMath::ClosestPointOnTetrahedronToPoint() : impossible result");
		break;
	}

	return Result;
}

void FMath::SphereDistToLine(FVector SphereOrigin, float SphereRadius, FVector LineOrigin, FVector NormalizedLineDir, FVector& OutClosestPoint)
{
	//const float A = NormalizedLineDir | NormalizedLineDir  (this is 1 because normalized)
	//solving quadratic formula in terms of t where closest point = LineOrigin + t * NormalizedLineDir
	const FVector LineOriginToSphereOrigin = SphereOrigin - LineOrigin;
	const FVector::FReal B = -2.f * (NormalizedLineDir | LineOriginToSphereOrigin);
	const FVector::FReal C = LineOriginToSphereOrigin.SizeSquared() - FMath::Square(SphereRadius);
	const FVector::FReal D	= FMath::Square(B) - 4.f * C;

	if( D <= UE_KINDA_SMALL_NUMBER )
	{
		// line is not intersecting sphere (or is tangent at one point if D == 0 )
		const FVector PointOnLine = LineOrigin + ( -B * 0.5f ) * NormalizedLineDir;
		OutClosestPoint = SphereOrigin + (PointOnLine - SphereOrigin).GetSafeNormal() * SphereRadius;
	}
	else
	{
		// Line intersecting sphere in 2 points. Pick closest to line origin.
		const FVector::FReal	E	= FMath::Sqrt(D);
		const FVector::FReal T1	= (-B + E) * 0.5f;
		const FVector::FReal T2	= (-B - E) * 0.5f;
		const FVector::FReal T	= FMath::Abs( T1 ) == FMath::Abs( T2 ) ? FMath::Abs( T1 ) : FMath::Abs( T1 ) < FMath::Abs( T2 ) ? T1 : T2;	// In the case where both points are exactly the same distance we take the one in the direction of LineDir

		OutClosestPoint	= LineOrigin + T * NormalizedLineDir;
	}
}

bool FMath::GetDistanceWithinConeSegment(FVector Point, FVector ConeStartPoint, FVector ConeLine, float RadiusAtStart, float RadiusAtEnd, float &PercentageOut)
{
	check(RadiusAtStart >= 0.0f && RadiusAtEnd >= 0.0f && ConeLine.SizeSquared() > 0);
	// -- First we'll draw out a line from the ConeStartPoint down the ConeLine. We'll find the closest point on that line to Point.
	//    If we're outside the max distance, or behind the StartPoint, we bail out as that means we've no chance to be in the cone.

	FVector PointOnCone; // Stores the point on the cone's center line closest to our target point.

	const FVector::FReal Distance = FMath::PointDistToLine(Point, ConeLine, ConeStartPoint, PointOnCone); // distance is how far from the viewline we are

	PercentageOut = 0.0; // start assuming we're outside cone until proven otherwise.

	const FVector VectToStart = ConeStartPoint - PointOnCone;
	const FVector VectToEnd = (ConeStartPoint + ConeLine) - PointOnCone;
	
	const FVector::FReal ConeLengthSqr = ConeLine.SizeSquared();
	const FVector::FReal DistToStartSqr = VectToStart.SizeSquared();
	const FVector::FReal DistToEndSqr = VectToEnd.SizeSquared();

	if (DistToStartSqr > ConeLengthSqr || DistToEndSqr > ConeLengthSqr)
	{
		//Outside cone
		return false;
	}

	const FVector::FReal PercentAlongCone = FMath::Sqrt(DistToStartSqr) / FMath::Sqrt(ConeLengthSqr); // don't have to catch outside 0->1 due to above code (saves 2 sqrts if outside)
	const FVector::FReal RadiusAtPoint = RadiusAtStart + ((RadiusAtEnd - RadiusAtStart) * PercentAlongCone);

	if(Distance > RadiusAtPoint) // target is farther from the line than the radius at that distance)
		return false;

	PercentageOut = RadiusAtPoint > 0.0f ? float((RadiusAtPoint - Distance) / RadiusAtPoint) : 1.0f;

	return true;
}

bool FMath::PointsAreCoplanar(const TArray<FVector>& Points, const float Tolerance)
{
	//less than 4 points = coplanar
	if (Points.Num() < 4)
	{
		return true;
	}

	//Get the Normal for plane determined by first 3 points
	const FVector Normal = FVector::CrossProduct(Points[2] - Points[0], Points[1] - Points[0]).GetSafeNormal();

	const int32 Total = Points.Num();
	for (int32 v = 3; v < Total; v++)
	{
		//Abs of PointPlaneDist, dist should be 0
		if (FMath::Abs(FVector::PointPlaneDist(Points[v], Points[0], Normal)) > Tolerance)
		{
			return false;
		}
	}

	return true;
}

bool FMath::GetDotDistance
( 
			FVector2D	&OutDotDist, 
	const	FVector		&Direction, 
	const	FVector		&AxisX, 
	const	FVector		&AxisY, 
	const	FVector		&AxisZ 	
)
{
	const FVector NormalDir = Direction.GetSafeNormal();

	// Find projected point (on AxisX and AxisY, remove AxisZ component)
	const FVector NoZProjDir = (NormalDir - (NormalDir | AxisZ) * AxisZ).GetSafeNormal();
	
	// Figure out if projection is on right or left.
	const FVector::FReal AzimuthSign = ( (NoZProjDir | AxisY) < 0.f ) ? -1.f : 1.f;

	using FVec2Real = decltype(FVector2D::X);
	OutDotDist.Y = FVec2Real(NormalDir | AxisZ);
	const FVector::FReal DirDotX	= NoZProjDir | AxisX;
	OutDotDist.X = FVec2Real(AzimuthSign * FMath::Abs(DirDotX));

	return (DirDotX >= 0.f );
}

FVector2D FMath::GetAzimuthAndElevation
(
	const FVector &Direction, 
	const FVector &AxisX, 
	const FVector &AxisY, 
	const FVector &AxisZ 	
)
{
	const FVector NormalDir = Direction.GetSafeNormal();
	// Find projected point (on AxisX and AxisY, remove AxisZ component)
	const FVector NoZProjDir = (NormalDir - (NormalDir | AxisZ) * AxisZ).GetSafeNormal();
	// Figure out if projection is on right or left.
	const FVector::FReal AzimuthSign = ((NoZProjDir | AxisY) < 0.f) ? -1.f : 1.f;
	const FVector::FReal ElevationSin = NormalDir | AxisZ;
	const FVector::FReal AzimuthCos = NoZProjDir | AxisX;

	// Convert to Angles in Radian.
	using FVec2Real = decltype(FVector2D::X);
	return FVector2D(FVec2Real(FMath::Acos(AzimuthCos) * AzimuthSign), (FVec2Real)FMath::Asin(ElevationSin));
}

CORE_API FVector FMath::VInterpNormalRotationTo(const FVector& Current, const FVector& Target, float DeltaTime, float RotationSpeedDegrees)
{
	// Find delta rotation between both normals.
	FQuat DeltaQuat = FQuat::FindBetween(Current, Target);

	// Decompose into an axis and angle for rotation
	FVector DeltaAxis(0.f);
	FQuat::FReal DeltaAngle = 0.f;
	DeltaQuat.ToAxisAndAngle(DeltaAxis, DeltaAngle);

	// Find rotation step for this frame
	const float RotationStepRadians = RotationSpeedDegrees * (UE_PI / 180) * DeltaTime;

	if( FMath::Abs(DeltaAngle) > RotationStepRadians )
	{
		DeltaAngle = FMath::Clamp<FQuat::FReal>(DeltaAngle, -RotationStepRadians, RotationStepRadians);
		DeltaQuat = FQuat(DeltaAxis, DeltaAngle);
		return DeltaQuat.RotateVector(Current);
	}
	return Target;
}

CORE_API FVector FMath::VInterpConstantTo(const FVector& Current, const FVector& Target, float DeltaTime, float InterpSpeed)
{
	const FVector Delta = Target - Current;
	const FVector::FReal DeltaM = Delta.Size();
	const FVector::FReal MaxStep = InterpSpeed * DeltaTime;

	if( DeltaM > MaxStep )
	{
		if( MaxStep > 0.f )
		{
			const FVector DeltaN = Delta / DeltaM;
			return Current + DeltaN * MaxStep;
		}
		else
		{
			return Current;
		}
	}

	return Target;
}

CORE_API FVector FMath::VInterpTo( const FVector& Current, const FVector& Target, float DeltaTime, float InterpSpeed )
{
	// If no interp speed, jump to target value
	if( InterpSpeed <= 0.f )
	{
		return Target;
	}

	// Distance to reach
	const FVector Dist = Target - Current;

	// If distance is too small, just set the desired location
	if( Dist.SizeSquared() < UE_KINDA_SMALL_NUMBER )
	{
		return Target;
	}

	// Delta Move, Clamp so we do not over shoot.
	const FVector	DeltaMove = Dist * FMath::Clamp<float>(DeltaTime * InterpSpeed, 0.f, 1.f);

	return Current + DeltaMove;
}

CORE_API FVector2D FMath::Vector2DInterpConstantTo( const FVector2D& Current, const FVector2D& Target, float DeltaTime, float InterpSpeed )
{
	const FVector2D Delta = Target - Current;
	const FVector2D::FReal DeltaM = Delta.Size();
	const float MaxStep = InterpSpeed * DeltaTime;

	if( DeltaM > MaxStep )
	{
		if( MaxStep > 0.f )
		{
			const FVector2D DeltaN = Delta / DeltaM;
			return Current + DeltaN * MaxStep;
		}
		else
		{
			return Current;
		}
	}

	return Target;
}

template <typename T>
CORE_API UE::Math::TVector2<T> FMath::Vector2DInterpTo( const UE::Math::TVector2<T>& Current, const UE::Math::TVector2<T>& Target, float DeltaTime, float InterpSpeed )
{
	if( InterpSpeed <= 0.f )
	{
		return Target;
	}

	const UE::Math::TVector2<T> Dist = Target - Current;
	if( Dist.SizeSquared() < static_cast<typename UE::Math::TVector2<T>::FReal>(UE_KINDA_SMALL_NUMBER) )
	{
		return Target;
	}

	const UE::Math::TVector2<T> DeltaMove = Dist * static_cast<typename UE::Math::TVector2<T>::FReal>(FMath::Clamp(DeltaTime * InterpSpeed, 0.f, 1.f));
	return Current + DeltaMove;
}

CORE_API FRotator FMath::RInterpConstantTo( const FRotator& Current, const FRotator& Target, float DeltaTime, float InterpSpeed )
{
	// if DeltaTime is 0, do not perform any interpolation (Location was already calculated for that frame)
	if( DeltaTime == 0.f || Current == Target )
	{
		return Current;
	}

	// If no interp speed, jump to target value
	if( InterpSpeed <= 0.f )
	{
		return Target;
	}

	const float DeltaInterpSpeed = InterpSpeed * DeltaTime;
	
	const FRotator DeltaMove = (Target - Current).GetNormalized();
	FRotator Result = Current;
	Result.Pitch += FMath::Clamp(DeltaMove.Pitch, -DeltaInterpSpeed, DeltaInterpSpeed);
	Result.Yaw += FMath::Clamp(DeltaMove.Yaw, -DeltaInterpSpeed, DeltaInterpSpeed);
	Result.Roll += FMath::Clamp(DeltaMove.Roll, -DeltaInterpSpeed, DeltaInterpSpeed);
	return Result.GetNormalized();
}

CORE_API FRotator FMath::RInterpTo( const FRotator& Current, const FRotator& Target, float DeltaTime, float InterpSpeed)
{
	// if DeltaTime is 0, do not perform any interpolation (Location was already calculated for that frame)
	if( DeltaTime == 0.f || Current == Target )
	{
		return Current;
	}

	// If no interp speed, jump to target value
	if( InterpSpeed <= 0.f )
	{
		return Target;
	}

	const float DeltaInterpSpeed = InterpSpeed * DeltaTime;

	const FRotator Delta = (Target - Current).GetNormalized();
	
	// If steps are too small, just return Target and assume we have reached our destination.
	if (Delta.IsNearlyZero())
	{
		return Target;
	}

	// Delta Move, Clamp so we do not over shoot.
	const FRotator DeltaMove = Delta * FMath::Clamp<float>(DeltaInterpSpeed, 0.f, 1.f);
	return (Current + DeltaMove).GetNormalized();
}

/** Interpolate Linear Color from Current to Target. Scaled by distance to Target, so it has a strong start speed and ease out. */
CORE_API FLinearColor FMath::CInterpTo(const FLinearColor& Current, const FLinearColor& Target, float DeltaTime, float InterpSpeed)
{
	// If no interp speed, jump to target value
	if (InterpSpeed <= 0.f)
	{
		return Target;
	}

	// Difference between colors
	const float Dist = FLinearColor::Dist(Target, Current);

	// If distance is too small, just set the desired color
	if (Dist < UE_KINDA_SMALL_NUMBER)
	{
		return Target;
	}

	// Delta change, Clamp so we do not over shoot.
	const FLinearColor DeltaMove = (Target - Current) * FMath::Clamp<float>(DeltaTime * InterpSpeed, 0.f, 1.f);

	return Current + DeltaMove;
}

template< class T >
CORE_API UE::Math::TQuat<T> FMath::QInterpConstantTo(const UE::Math::TQuat<T>& Current, const UE::Math::TQuat<T>& Target, float DeltaTime, float InterpSpeed)
{
	// If no interp speed, jump to target value
	if (InterpSpeed <= 0.f)
	{
		return Target;
	}

	// If the values are nearly equal, just return Target and assume we have reached our destination.
	if (Current.Equals(Target))
	{
		return Target;
	}

	float DeltaInterpSpeed = FMath::Clamp<float>(DeltaTime * InterpSpeed, 0.f, 1.f);
	float AngularDistance = FMath::Max<float>(UE_SMALL_NUMBER, (float)(Target.AngularDistance(Current)));
	float Alpha = FMath::Clamp<float>(DeltaInterpSpeed / AngularDistance, 0.f, 1.f);

	return UE::Math::TQuat<T>::Slerp(Current, Target, Alpha);
}

// Instantiate for linker
template CORE_API UE::Math::TQuat<float> FMath::QInterpConstantTo<float>(const UE::Math::TQuat<float>& Current, const UE::Math::TQuat<float>& Target, float DeltaTime, float InterpSpeed);
template CORE_API UE::Math::TQuat<double> FMath::QInterpConstantTo<double>(const UE::Math::TQuat<double>& Current, const UE::Math::TQuat<double>& Target, float DeltaTime, float InterpSpeed);


template< class T >
CORE_API UE::Math::TQuat<T> FMath::QInterpTo(const UE::Math::TQuat<T>& Current, const UE::Math::TQuat<T>& Target, float DeltaTime, float InterpSpeed)
{
	// If no interp speed, jump to target value
	if (InterpSpeed <= 0.f)
	{
		return Target;
	}

	// If the values are nearly equal, just return Target and assume we have reached our destination.
	if (Current.Equals(Target))
	{
		return Target;
	}

	return UE::Math::TQuat<T>::Slerp(Current, Target, FMath::Clamp<float>(InterpSpeed * DeltaTime, 0.f, 1.f));
}

// Instantiate for linker
template CORE_API UE::Math::TQuat<float> FMath::QInterpTo<float>(const UE::Math::TQuat<float>& Current, const UE::Math::TQuat<float>& Target, float DeltaTime, float InterpSpeed);
template CORE_API UE::Math::TQuat<double> FMath::QInterpTo<double>(const UE::Math::TQuat<double>& Current, const UE::Math::TQuat<double>& Target, float DeltaTime, float InterpSpeed);


CORE_API float ClampFloatTangent( float PrevPointVal, float PrevTime, float CurPointVal, float CurTime, float NextPointVal, float NextTime )
{
	const float PrevToNextTimeDiff = FMath::Max< float >( UE_KINDA_SMALL_NUMBER, NextTime - PrevTime );
	const float PrevToCurTimeDiff = FMath::Max< float >( UE_KINDA_SMALL_NUMBER, CurTime - PrevTime );
	const float CurToNextTimeDiff = FMath::Max< float >( UE_KINDA_SMALL_NUMBER, NextTime - CurTime );

	float OutTangentVal = 0.0f;

	const float PrevToNextHeightDiff = NextPointVal - PrevPointVal;
	const float PrevToCurHeightDiff = CurPointVal - PrevPointVal;
	const float CurToNextHeightDiff = NextPointVal - CurPointVal;

	// Check to see if the current point is crest
	if( ( PrevToCurHeightDiff >= 0.0f && CurToNextHeightDiff <= 0.0f ) ||
		( PrevToCurHeightDiff <= 0.0f && CurToNextHeightDiff >= 0.0f ) )
	{
		// Neighbor points are both both on the same side, so zero out the tangent
		OutTangentVal = 0.0f;
	}
	else
	{
		// The three points form a slope

		// Constants
		const float ClampThreshold = 0.333f;

		// Compute height deltas
		const float CurToNextTangent = CurToNextHeightDiff / CurToNextTimeDiff;
		const float PrevToCurTangent = PrevToCurHeightDiff / PrevToCurTimeDiff;
		const float PrevToNextTangent = PrevToNextHeightDiff / PrevToNextTimeDiff;

		// Default to not clamping
		const float UnclampedTangent = PrevToNextTangent;
		float ClampedTangent = UnclampedTangent;

		const float LowerClampThreshold = ClampThreshold;
		const float UpperClampThreshold = 1.0f - ClampThreshold;

		// @todo: Would we get better results using percentange of TIME instead of HEIGHT?
		const float CurHeightAlpha = PrevToCurHeightDiff / PrevToNextHeightDiff;

		if( PrevToNextHeightDiff > 0.0f )
		{
			if( CurHeightAlpha < LowerClampThreshold )
			{
				// 1.0 = maximum clamping (flat), 0.0 = minimal clamping (don't touch)
				const float ClampAlpha = 1.0f - CurHeightAlpha / ClampThreshold;
				const float LowerClamp = FMath::Lerp( PrevToNextTangent, PrevToCurTangent, ClampAlpha );
				ClampedTangent = FMath::Min( ClampedTangent, LowerClamp );
			}

			if( CurHeightAlpha > UpperClampThreshold )
			{
				// 1.0 = maximum clamping (flat), 0.0 = minimal clamping (don't touch)
				const float ClampAlpha = ( CurHeightAlpha - UpperClampThreshold ) / ClampThreshold;
				const float UpperClamp = FMath::Lerp( PrevToNextTangent, CurToNextTangent, ClampAlpha );
				ClampedTangent = FMath::Min( ClampedTangent, UpperClamp );
			}
		}
		else
		{

			if( CurHeightAlpha < LowerClampThreshold )
			{
				// 1.0 = maximum clamping (flat), 0.0 = minimal clamping (don't touch)
				const float ClampAlpha = 1.0f - CurHeightAlpha / ClampThreshold;
				const float LowerClamp = FMath::Lerp( PrevToNextTangent, PrevToCurTangent, ClampAlpha );
				ClampedTangent = FMath::Max( ClampedTangent, LowerClamp );
			}

			if( CurHeightAlpha > UpperClampThreshold )
			{
				// 1.0 = maximum clamping (flat), 0.0 = minimal clamping (don't touch)
				const float ClampAlpha = ( CurHeightAlpha - UpperClampThreshold ) / ClampThreshold;
				const float UpperClamp = FMath::Lerp( PrevToNextTangent, CurToNextTangent, ClampAlpha );
				ClampedTangent = FMath::Max( ClampedTangent, UpperClamp );
			}
		}

		OutTangentVal = ClampedTangent;
	}

	return OutTangentVal;
}

FVector FMath::VRandCone(FVector const& Dir, float ConeHalfAngleRad)
{
	if (ConeHalfAngleRad > 0.f)
	{
		float const RandU = FMath::FRand();
		float const RandV = FMath::FRand();

		// Get spherical coords that have an even distribution over the unit sphere
		// Method described at http://mathworld.wolfram.com/SpherePointPicking.html	
		float Theta = 2.f * UE_PI * RandU;
		float Phi = FMath::Acos((2.f * RandV) - 1.f);

		// restrict phi to [0, ConeHalfAngleRad]
		// this gives an even distribution of points on the surface of the cone
		// centered at the origin, pointing upward (z), with the desired angle
		Phi = FMath::Fmod(Phi, ConeHalfAngleRad);

		// get axes we need to rotate around
		FMatrix const DirMat = FRotationMatrix(Dir.Rotation());
		// note the axis translation, since we want the variation to be around X
		FVector const DirZ = DirMat.GetScaledAxis( EAxis::X );		
		FVector const DirY = DirMat.GetScaledAxis( EAxis::Y );

		FVector Result = Dir.RotateAngleAxis(Phi * 180.f / UE_PI, DirY);
		Result = Result.RotateAngleAxis(Theta * 180.f / UE_PI, DirZ);

		// ensure it's a unit vector (might not have been passed in that way)
		Result = Result.GetSafeNormal();
		
		return Result;
	}
	else
	{
		return Dir.GetSafeNormal();
	}
}

FVector FMath::VRandCone(FVector const& Dir, float HorizontalConeHalfAngleRad, float VerticalConeHalfAngleRad)
{
	if ( (VerticalConeHalfAngleRad > 0.f) && (HorizontalConeHalfAngleRad > 0.f) )
	{
		float const RandU = FMath::FRand();
		float const RandV = FMath::FRand();

		// Get spherical coords that have an even distribution over the unit sphere
		// Method described at http://mathworld.wolfram.com/SpherePointPicking.html	
		float Theta = 2.f * UE_PI * RandU;
		float Phi = FMath::Acos((2.f * RandV) - 1.f);

		// restrict phi to [0, ConeHalfAngleRad]
		// where ConeHalfAngleRad is now a function of Theta
		// (specifically, radius of an ellipse as a function of angle)
		// function is ellipse function (x/a)^2 + (y/b)^2 = 1, converted to polar coords
		float ConeHalfAngleRad = FMath::Square(FMath::Cos(Theta) / VerticalConeHalfAngleRad) + FMath::Square(FMath::Sin(Theta) / HorizontalConeHalfAngleRad);
		ConeHalfAngleRad = FMath::Sqrt(1.f / ConeHalfAngleRad);

		// clamp to make a cone instead of a sphere
		Phi = FMath::Fmod(Phi, ConeHalfAngleRad);

		// get axes we need to rotate around
		FMatrix const DirMat = FRotationMatrix(Dir.Rotation());
		// note the axis translation, since we want the variation to be around X
		FVector const DirZ = DirMat.GetScaledAxis( EAxis::X );		
		FVector const DirY = DirMat.GetScaledAxis( EAxis::Y );

		FVector Result = Dir.RotateAngleAxis(Phi * 180.f / UE_PI, DirY);
		Result = Result.RotateAngleAxis(Theta * 180.f / UE_PI, DirZ);

		// ensure it's a unit vector (might not have been passed in that way)
		Result = Result.GetSafeNormal();

		return Result;
	}
	else
	{
		return Dir.GetSafeNormal();
	}
}

FVector2D FMath::RandPointInCircle(float CircleRadius)
{
	FVector2D Point;
	FVector2D::FReal L;

	do
	{
		// Check random vectors in the unit circle so result is statistically uniform.
		Point.X = FRand() * 2.f - 1.f;
		Point.Y = FRand() * 2.f - 1.f;
		L = Point.SizeSquared();
	}
	while (L > 1.0f);

	return Point * CircleRadius;
}

FVector FMath::RandPointInBox(const FBox& Box)
{
	return FVector(	FRandRange(Box.Min.X, Box.Max.X),
					FRandRange(Box.Min.Y, Box.Max.Y),
					FRandRange(Box.Min.Z, Box.Max.Z) );
}

FVector FMath::GetReflectionVector(const FVector& Direction, const FVector& SurfaceNormal)
{
	FVector SafeNormal(SurfaceNormal.GetSafeNormal());

	return Direction - 2 * (Direction | SafeNormal) * SafeNormal;
}

float FMath::TruncateToHalfIfClose(float F, float Tolerance)
{
	float ValueToFudgeIntegralPart = 0.0f;
	float ValueToFudgeFractionalPart = FMath::Modf(F, &ValueToFudgeIntegralPart);
	if (F < 0.0f)
	{
		return ValueToFudgeIntegralPart + (FMath::IsNearlyEqual(ValueToFudgeFractionalPart, -0.5f, Tolerance) ? -0.5f : ValueToFudgeFractionalPart);
	}
	else
	{
		return ValueToFudgeIntegralPart + (FMath::IsNearlyEqual(ValueToFudgeFractionalPart, 0.5f, Tolerance) ? 0.5f : ValueToFudgeFractionalPart);
	}
}

double FMath::TruncateToHalfIfClose(double F, double Tolerance)
{
	double ValueToFudgeIntegralPart = 0.0;
	double ValueToFudgeFractionalPart = FMath::Modf(F, &ValueToFudgeIntegralPart);
	if (F < 0.0)
	{
		return ValueToFudgeIntegralPart + (FMath::IsNearlyEqual(ValueToFudgeFractionalPart, -0.5, Tolerance) ? -0.5 : ValueToFudgeFractionalPart);
	}
	else
	{
		return ValueToFudgeIntegralPart + (FMath::IsNearlyEqual(ValueToFudgeFractionalPart, 0.5, Tolerance) ? 0.5 : ValueToFudgeFractionalPart);
	}
}

float FMath::RoundHalfToEven(float F)
{
	F = FMath::TruncateToHalfIfClose(F);

	const bool bIsNegative = F < 0.0f;
	const bool bValueIsEven = static_cast<uint32>(FloorToFloat(((bIsNegative) ? -F : F))) % 2 == 0;
	if (bValueIsEven)
	{
		// Round towards value (eg, value is -2.5 or 2.5, and should become -2 or 2)
		return (bIsNegative) ? FloorToFloat(F + 0.5f) : CeilToFloat(F - 0.5f);
	}
	else
	{
		// Round away from value (eg, value is -3.5 or 3.5, and should become -4 or 4)
		return (bIsNegative) ? CeilToFloat(F - 0.5f) : FloorToFloat(F + 0.5f);
	}
}

double FMath::RoundHalfToEven(double F)
{
	F = FMath::TruncateToHalfIfClose(F);

	const bool bIsNegative = F < 0.0;
	const bool bValueIsEven = static_cast<uint64>(FMath::FloorToDouble(((bIsNegative) ? -F : F))) % 2 == 0;
	if (bValueIsEven)
	{
		// Round towards value (eg, value is -2.5 or 2.5, and should become -2 or 2)
		return (bIsNegative) ? FloorToDouble(F + 0.5) : CeilToDouble(F - 0.5);
	}
	else
	{
		// Round away from value (eg, value is -3.5 or 3.5, and should become -4 or 4)
		return (bIsNegative) ? CeilToDouble(F - 0.5) : FloorToDouble(F + 0.5);
	}
}

float FMath::RoundHalfFromZero(float F)
{
	float ValueToFudgeIntegralPart = 0.0f;
	float ValueToFudgeFractionalPart = FMath::Modf(F, &ValueToFudgeIntegralPart);

	if (F < 0.0f)
	{
		return ValueToFudgeFractionalPart > -0.5f ? ValueToFudgeIntegralPart : ValueToFudgeIntegralPart - 1.0f;
	}
	else
	{
		return ValueToFudgeFractionalPart < 0.5f ? ValueToFudgeIntegralPart : ValueToFudgeIntegralPart + 1.0f;
	}
}

double FMath::RoundHalfFromZero(double F)
{
	double ValueToFudgeIntegralPart = 0.0;
	double ValueToFudgeFractionalPart = FMath::Modf(F, &ValueToFudgeIntegralPart);

	if (F < 0.0)
	{
		return ValueToFudgeFractionalPart > -0.5 ? ValueToFudgeIntegralPart : ValueToFudgeIntegralPart - 1.0;
	}
	else
	{
		return ValueToFudgeFractionalPart < 0.5 ? ValueToFudgeIntegralPart : ValueToFudgeIntegralPart + 1.0f;
	}
}

float FMath::RoundHalfToZero(float F)
{
	float ValueToFudgeIntegralPart = 0.0f;
	float ValueToFudgeFractionalPart = FMath::Modf(F, &ValueToFudgeIntegralPart);

	if (F < 0.0f)
	{
		return ValueToFudgeFractionalPart < -0.5f ? ValueToFudgeIntegralPart - 1.0f : ValueToFudgeIntegralPart;
	}
	else
	{
		return ValueToFudgeFractionalPart > 0.5f ? ValueToFudgeIntegralPart + 1.0f : ValueToFudgeIntegralPart;
	}
}

double FMath::RoundHalfToZero(double F)
{
	double ValueToFudgeIntegralPart = 0.0;
	double ValueToFudgeFractionalPart = FMath::Modf(F, &ValueToFudgeIntegralPart);

	if (F < 0.0)
	{
		return ValueToFudgeFractionalPart < -0.5 ? ValueToFudgeIntegralPart - 1.0 : ValueToFudgeIntegralPart;
	}
	else
	{
		return ValueToFudgeFractionalPart > 0.5 ? ValueToFudgeIntegralPart + 1.0 : ValueToFudgeIntegralPart;
	}
}

FString FMath::FormatIntToHumanReadable(int32 Val)
{
	FString Src = *FString::Printf(TEXT("%i"), Val);
	FString Dst;

	while (Src.Len() > 3 && Src[Src.Len() - 4] != TEXT('-'))
	{
		Dst = FString::Printf(TEXT(",%s%s"), *Src.Right(3), *Dst);
		Src.LeftInline(Src.Len() - 3, EAllowShrinking::No);
	}

	Dst = Src + Dst;

	return Dst;
}

bool FMath::MemoryTest( void* BaseAddress, uint32 NumBytes )
{
	volatile uint32* Ptr;
	uint32 NumDwords = NumBytes / 4;
	uint32 TestWords[2] = { 0xdeadbeef, 0x1337c0de };
	bool bSucceeded = true;

	for ( int32 TestIndex=0; TestIndex < 2; ++TestIndex )
	{
		// Fill the memory with a pattern.
		Ptr = (uint32*) BaseAddress;
		for ( uint32 Index=0; Index < NumDwords; ++Index )
		{
			*Ptr = TestWords[TestIndex];
			Ptr++;
		}

		// Check that each uint32 is still ok and overwrite it with the complement.
		Ptr = (uint32*) BaseAddress;
		for ( uint32 Index=0; Index < NumDwords; ++Index )
		{
			if ( *Ptr != TestWords[TestIndex] )
			{
				FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Failed memory test at 0x%08x, wrote: 0x%08x, read: 0x%08x\n"), Ptr, TestWords[TestIndex], *Ptr );
				bSucceeded = false;
			}
			*Ptr = ~TestWords[TestIndex];
			Ptr++;
		}

		// Check again, now going backwards in memory.
		Ptr = ((uint32*) BaseAddress) + NumDwords;
		for ( uint32 Index=0; Index < NumDwords; ++Index )
		{
			Ptr--;
			if ( *Ptr != ~TestWords[TestIndex] )
			{
				FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Failed memory test at 0x%08x, wrote: 0x%08x, read: 0x%08x\n"), Ptr, ~TestWords[TestIndex], *Ptr );
				bSucceeded = false;
			}
			*Ptr = TestWords[TestIndex];
		}
	}

	return bSucceeded;
}

/**
 * Converts a string to it's numeric equivalent, ignoring whitespace.
 * "123  45" - becomes 12,345
 *
 * @param	Value	The string to convert.
 * @return			The converted value.
 */
float Val(const FString& Value)
{
	float RetValue = 0;

	for( int32 x = 0 ; x < Value.Len() ; x++ )
	{
		FString Char = Value.Mid(x, 1);

		if( Char >= TEXT("0") && Char <= TEXT("9") )
		{
			RetValue *= 10;
			RetValue += (float)FCString::Atoi( *Char );
		}
		else 
		{
			if( Char != TEXT(" ") )
			{
				break;
			}
		}
	}

	return RetValue;
}

FString GrabChar( FString* pStr )
{
	FString GrabChar;
	if( pStr->Len() )
	{
		do
		{		
			GrabChar = pStr->Left(1);
			*pStr = pStr->Mid(1);
		} while( GrabChar == TEXT(" ") );
	}
	else
	{
		GrabChar = TEXT("");
	}

	return GrabChar;
}

bool SubEval( FString* pStr, float* pResult, int32 Prec )
{
	FString c;
	float V, W, N;

	V = W = N = 0.0f;

	c = GrabChar(pStr);

	if( (c >= TEXT("0") && c <= TEXT("9")) || c == TEXT(".") )	// Number
	{
		V = 0;
		while(c >= TEXT("0") && c <= TEXT("9"))
		{
			V = V * 10 + Val(c);
			c = GrabChar(pStr);
		}

		if( c == TEXT(".") )
		{
			N = 0.1f;
			c = GrabChar(pStr);

			while(c >= TEXT("0") && c <= TEXT("9"))
			{
				V = V + N * Val(c);
				N = N / 10.0f;
				c = GrabChar(pStr);
			}
		}
	}
	else if( c == TEXT("("))									// Opening parenthesis
	{
		if( !SubEval(pStr, &V, 0) )
		{
			return 0;
		}
		c = GrabChar(pStr);
	}
	else if( c == TEXT("-") )									// Negation
	{
		if( !SubEval(pStr, &V, 1000) )
		{
			return 0;
		}
		V = -V;
		c = GrabChar(pStr);
	}
	else if( c == TEXT("+"))									// Positive
	{
		if( !SubEval(pStr, &V, 1000) )
		{
			return 0;
		}
		c = GrabChar(pStr);
	}
	else if( c == TEXT("@") )									// Square root
	{
		if( !SubEval(pStr, &V, 1000) )
		{
			return 0;
		}

		if( V < 0 )
		{
			UE_LOGF(LogUnrealMath, Log, "Expression Error : Can't take square root of negative number");
			return 0;
		}
		else
		{
			V = FMath::Sqrt(V);
		}

		c = GrabChar(pStr);
	}
	else														// Error
	{
		UE_LOGF(LogUnrealMath, Log, "Expression Error : No value recognized");
		return 0;
	}
PrecLoop:
	if( c == TEXT("") )
	{
		*pResult = V;
		return 1;
	}
	else if( c == TEXT(")") )
	{
		*pStr = FString(")") + *pStr;
		*pResult = V;
		return 1;
	}
	else if( c == TEXT("+") )
	{
		if( Prec > 1 )
		{
			*pResult = V;
			*pStr = c + *pStr;
			return 1;
		}
		else
		{
			if( SubEval(pStr, &W, 2) )
			{
				V = V + W;
				c = GrabChar(pStr);
				goto PrecLoop;
			}
			else
			{
				return 0;
			}
		}
	}
	else if( c == TEXT("-") )
	{
		if( Prec > 1 )
		{
			*pResult = V;
			*pStr = c + *pStr;
			return 1;
		}
		else
		{
			if( SubEval(pStr, &W, 2) )
			{
				V = V - W;
				c = GrabChar(pStr);
				goto PrecLoop;
			}
			else
			{
				return 0;
			}
		}
	}
	else if( c == TEXT("/") )
	{
		if( Prec > 2 )
		{
			*pResult = V;
			*pStr = c + *pStr;
			return 1;
		}
		else
		{
			if( SubEval(pStr, &W, 3) )
			{
				if( W == 0 )
				{
					UE_LOGF(LogUnrealMath, Log, "Expression Error : Division by zero isn't allowed");
					return 0;
				}
				else
				{
					V = V / W;
					c = GrabChar(pStr);
					goto PrecLoop;
				}
			}
			else
			{
				return 0;
			}
		}
	}
	else if( c == TEXT("%") )
	{
		if( Prec > 2 )
		{
			*pResult = V;
			*pStr = c + *pStr;
			return 1;
		}
		else
		{
			if( SubEval(pStr, &W, 3) )
			{
				if( W == 0 )
				{
					UE_LOGF(LogUnrealMath, Log, "Expression Error : Modulo zero isn't allowed");
					return 0;
				}
				else
				{
					V = (float)((int32)V % (int32)W);
					c = GrabChar(pStr);
					goto PrecLoop;
				}
			}
			else
			{
				return 0;
			}
		}
	}
	else if( c == TEXT("*") )
	{
		if( Prec > 3 )
		{
			*pResult = V;
			*pStr = c + *pStr;
			return 1;
		}
		else
		{
			if( SubEval(pStr, &W, 4) )
			{
				V = V * W;
				c = GrabChar(pStr);
				goto PrecLoop;
			}
			else
			{
				return 0;
			}
		}
	}
	else
	{
		UE_LOGF(LogUnrealMath, Log, "Expression Error : Unrecognized Operator");
		return 0;
	}
}

bool FMath::Eval( FString Str, float& OutValue )
{
	bool bResult = true;

	// Check for a matching number of brackets right up front.
	int32 Brackets = 0;
	for( int32 x = 0 ; x < Str.Len() ; x++ )
	{
		if( Str.Mid(x,1) == TEXT("(") )
		{
			Brackets++;
		}
		if( Str.Mid(x,1) == TEXT(")") )
		{
			Brackets--;
		}
	}

	if( Brackets != 0 )
	{
		UE_LOGF(LogUnrealMath, Log, "Expression Error : Mismatched brackets");
		bResult = false;
	}

	else
	{
		if( !SubEval( &Str, &OutValue, 0 ) )
		{
			UE_LOGF(LogUnrealMath, Log, "Expression Error : Error in expression");
			bResult = false;
		}
	}

	return bResult;
}

void FMath::WindRelativeAnglesDegrees(float InAngle0, float& InOutAngle1)
{
	const float Diff = InAngle0 - InOutAngle1;
	const float AbsDiff = Abs(Diff);
	if (AbsDiff > 180.0f)
	{
		InOutAngle1 += 360.0f * Sign(Diff) * FloorToFloat((AbsDiff / 360.0f) + 0.5f);
	}
}

void FMath::WindRelativeAnglesDegrees(double InAngle0, double& InOutAngle1)
{
	const double Diff = InAngle0 - InOutAngle1;
	const double AbsDiff = Abs(Diff);
	if (AbsDiff > 180.0)
	{
		InOutAngle1 += 360.0 * Sign(Diff) * FloorToDouble((AbsDiff / 360.0) + 0.5);
	}
}

float FMath::FixedTurn(float InCurrent, float InDesired, float InDeltaRate)
{
	if (InDeltaRate == 0.f)
	{
		return FRotator3f::ClampAxis(InCurrent);
	}

	if (InDeltaRate >= 360.f)
	{
		return FRotator3f::ClampAxis(InDesired);
	}

	float result = FRotator3f::ClampAxis(InCurrent);
	InCurrent = result;
	InDesired = FRotator3f::ClampAxis(InDesired);

	if (InCurrent > InDesired)
	{
		if (InCurrent - InDesired < 180.f)
			result -= FMath::Min((InCurrent - InDesired), FMath::Abs(InDeltaRate));
		else
			result += FMath::Min((InDesired + 360.f - InCurrent), FMath::Abs(InDeltaRate));
	}
	else
	{
		if (InDesired - InCurrent < 180.f)
			result += FMath::Min((InDesired - InCurrent), FMath::Abs(InDeltaRate));
		else
			result -= FMath::Min((InCurrent + 360.f - InDesired), FMath::Abs(InDeltaRate));
	}
	return FRotator3f::ClampAxis(result);
}

void FMath::ApplyScaleToFloat(float& Dst, const FVector& DeltaScale, float Magnitude)
{
	const float Multiplier = ( DeltaScale.X > 0.0f || DeltaScale.Y > 0.0f || DeltaScale.Z > 0.0f ) ? Magnitude : -Magnitude;
	Dst += Multiplier * (float)DeltaScale.Size();
	Dst = FMath::Max( 0.0f, Dst );
}

// Implementation of 1D, 2D and 3D Perlin noise based on Ken Perlin's improved version https://mrl.nyu.edu/~perlin/noise/
// (See Random3.tps for additional third party software info.)
namespace FMathPerlinHelpers
{
	// random permutation of 256 numbers, repeated 2x
	static const int32 Permutation[512] = {
		63, 9, 212, 205, 31, 128, 72, 59, 137, 203, 195, 170, 181, 115, 165, 40, 116, 139, 175, 225, 132, 99, 222, 2, 41, 15, 197, 93, 169, 90, 228, 43, 221, 38, 206, 204, 73, 17, 97, 10, 96, 47, 32, 138, 136, 30, 219,
		78, 224, 13, 193, 88, 134, 211, 7, 112, 176, 19, 106, 83, 75, 217, 85, 0, 98, 140, 229, 80, 118, 151, 117, 251, 103, 242, 81, 238, 172, 82, 110, 4, 227, 77, 243, 46, 12, 189, 34, 188, 200, 161, 68, 76, 171, 194,
		57, 48, 247, 233, 51, 105, 5, 23, 42, 50, 216, 45, 239, 148, 249, 84, 70, 125, 108, 241, 62, 66, 64, 240, 173, 185, 250, 49, 6, 37, 26, 21, 244, 60, 223, 255, 16, 145, 27, 109, 58, 102, 142, 253, 120, 149, 160,
		124, 156, 79, 186, 135, 127, 14, 121, 22, 65, 54, 153, 91, 213, 174, 24, 252, 131, 192, 190, 202, 208, 35, 94, 231, 56, 95, 183, 163, 111, 147, 25, 67, 36, 92, 236, 71, 166, 1, 187, 100, 130, 143, 237, 178, 158,
		104, 184, 159, 177, 52, 214, 230, 119, 87, 114, 201, 179, 198, 3, 248, 182, 39, 11, 152, 196, 113, 20, 232, 69, 141, 207, 234, 53, 86, 180, 226, 74, 150, 218, 29, 133, 8, 44, 123, 28, 146, 89, 101, 154, 220, 126,
		155, 122, 210, 168, 254, 162, 129, 33, 18, 209, 61, 191, 199, 157, 245, 55, 164, 167, 215, 246, 144, 107, 235, 

		63, 9, 212, 205, 31, 128, 72, 59, 137, 203, 195, 170, 181, 115, 165, 40, 116, 139, 175, 225, 132, 99, 222, 2, 41, 15, 197, 93, 169, 90, 228, 43, 221, 38, 206, 204, 73, 17, 97, 10, 96, 47, 32, 138, 136, 30, 219,
		78, 224, 13, 193, 88, 134, 211, 7, 112, 176, 19, 106, 83, 75, 217, 85, 0, 98, 140, 229, 80, 118, 151, 117, 251, 103, 242, 81, 238, 172, 82, 110, 4, 227, 77, 243, 46, 12, 189, 34, 188, 200, 161, 68, 76, 171, 194,
		57, 48, 247, 233, 51, 105, 5, 23, 42, 50, 216, 45, 239, 148, 249, 84, 70, 125, 108, 241, 62, 66, 64, 240, 173, 185, 250, 49, 6, 37, 26, 21, 244, 60, 223, 255, 16, 145, 27, 109, 58, 102, 142, 253, 120, 149, 160,
		124, 156, 79, 186, 135, 127, 14, 121, 22, 65, 54, 153, 91, 213, 174, 24, 252, 131, 192, 190, 202, 208, 35, 94, 231, 56, 95, 183, 163, 111, 147, 25, 67, 36, 92, 236, 71, 166, 1, 187, 100, 130, 143, 237, 178, 158,
		104, 184, 159, 177, 52, 214, 230, 119, 87, 114, 201, 179, 198, 3, 248, 182, 39, 11, 152, 196, 113, 20, 232, 69, 141, 207, 234, 53, 86, 180, 226, 74, 150, 218, 29, 133, 8, 44, 123, 28, 146, 89, 101, 154, 220, 126,
		155, 122, 210, 168, 254, 162, 129, 33, 18, 209, 61, 191, 199, 157, 245, 55, 164, 167, 215, 246, 144, 107, 235
	};

	// Gradient functions for 1D, 2D and 3D Perlin noise

	FORCEINLINE float Grad1(int32 Hash, float X)
	{
		// Slicing Perlin's 3D improved noise would give us only scales of -1, 0 and 1; this looks pretty bad so let's use a different sampling
		static const float Grad1Scales[16] = {-8/8, -7/8., -6/8., -5/8., -4/8., -3/8., -2/8., -1/8., 1/8., 2/8., 3/8., 4/8., 5/8., 6/8., 7/8., 8/8};
		return Grad1Scales[Hash & 15] * X;
	}

	// Note: If you change the Grad2 or Grad3 functions, check that you don't change the range of the resulting noise as well; it should be (within floating point error) in the range of (-1, 1)
	FORCEINLINE float Grad2(int32 Hash, float X, float Y)
	{
		// corners and major axes (similar to the z=0 projection of the cube-edge-midpoint sampling from improved Perlin noise)
		switch (Hash & 7)
		{
		case 0: return X;
		case 1: return X + Y;
		case 2: return Y;
		case 3: return -X + Y;
		case 4: return -X;
		case 5: return -X - Y;
		case 6: return -Y;
		case 7: return X - Y;
		// can't happen
		default: return 0;
		}
	}

	FORCEINLINE float Grad3(int32 Hash, float X, float Y, float Z)
	{
		switch (Hash & 15)
		{
		// 12 cube midpoints
		case 0: return X + Z;
		case 1: return X + Y;
		case 2: return Y + Z;
		case 3: return -X + Y;
		case 4: return -X + Z;
		case 5: return -X - Y;
		case 6: return -Y + Z;
		case 7: return X - Y;
		case 8: return X - Z;
		case 9: return Y - Z;
		case 10: return -X - Z;
		case 11: return -Y - Z;
		// 4 vertices of regular tetrahedron
		case 12: return X + Y;
		case 13: return -X + Y;
		case 14: return -Y + Z;
		case 15: return -Y - Z;
		// can't happen
		default: return 0;
		}
	}

	// Curve w/ second derivative vanishing at 0 and 1, from Perlin's improved noise paper
	FORCEINLINE float SmoothCurve(float X)
	{
		return X * X * X * (X * (X * 6.0f - 15.0f) + 10.0f);
	}
};

float FMath::PerlinNoise1D(float X)
{
	using namespace FMathPerlinHelpers;

	const float Xfl = FMath::FloorToFloat(X);
	const int32 Xi = (int32)(Xfl) & 255;
	X -= Xfl;
	const float Xm1 = X - 1.0f;

	const int32 A = Permutation[Xi];
	const int32 B = Permutation[Xi + 1];

	const float U = SmoothCurve(X);

	// 2.0 factor to ensure (-1, 1) range
	return 2.0f * Lerp(Grad1(A, X), Grad1(B, Xm1), U);
}

float FMath::PerlinNoise2D(const FVector2D& Location)
{
	using namespace FMathPerlinHelpers;

	float Xfl = FMath::FloorToFloat((float)Location.X);		// LWC_TODO: Precision loss
	float Yfl = FMath::FloorToFloat((float)Location.Y);
	int32 Xi = (int32)(Xfl) & 255;
	int32 Yi = (int32)(Yfl) & 255;
	float X = (float)Location.X - Xfl;
	float Y = (float)Location.Y - Yfl;
	float Xm1 = X - 1.0f;
	float Ym1 = Y - 1.0f;

	const int32 *P = Permutation;
	int32 AA = P[Xi] + Yi;
	int32 AB = AA + 1;
	int32 BA = P[Xi + 1] + Yi;
	int32 BB = BA + 1;

	float U = SmoothCurve(X);
	float V = SmoothCurve(Y);

	// Note: Due to the choice of Grad2, this will be in the (-1,1) range with no additional scaling
	return Lerp(
			Lerp(Grad2(P[AA], X, Y), Grad2(P[BA], Xm1, Y), U),
			Lerp(Grad2(P[AB], X, Ym1), Grad2(P[BB], Xm1, Ym1), U),
			V);
}

float FMath::PerlinNoise3D(const FVector& Location)
{
	using namespace FMathPerlinHelpers;

	float Xfl = FMath::FloorToFloat((float)Location.X);		// LWC_TODO: Precision loss
	float Yfl = FMath::FloorToFloat((float)Location.Y);
	float Zfl = FMath::FloorToFloat((float)Location.Z);
	int32 Xi = (int32)(Xfl) & 255;
	int32 Yi = (int32)(Yfl) & 255;
	int32 Zi = (int32)(Zfl) & 255;
	float X = (float)Location.X - Xfl;
	float Y = (float)Location.Y - Yfl;
	float Z = (float)Location.Z - Zfl;
	float Xm1 = X - 1.0f;
	float Ym1 = Y - 1.0f;
	float Zm1 = Z - 1.0f;

	const int32 *P = Permutation;
	int32 A = P[Xi] + Yi;
	int32 AA = P[A] + Zi;	int32 AB = P[A + 1] + Zi;

	int32 B = P[Xi + 1] + Yi;
	int32 BA = P[B] + Zi;	int32 BB = P[B + 1] + Zi;

	float U = SmoothCurve(X);
	float V = SmoothCurve(Y);
	float W = SmoothCurve(Z);
	
	// Note: range is already approximately -1,1 because of the specific choice of direction vectors for the Grad3 function
	// This analysis (http://digitalfreepen.com/2017/06/20/range-perlin-noise.html) suggests scaling by 1/sqrt(3/4) * 1/maxGradientVectorLen, but the choice of gradient vectors makes this overly conservative
	// Scale factor of .97 is (1.0/the max values of a billion random samples); to be 100% sure about the range I also just Clamp it for now.
	return Clamp(0.97f *
		Lerp(Lerp(Lerp(Grad3(P[AA], X, Y, Z), Grad3(P[BA], Xm1, Y, Z), U),
				 Lerp(Grad3(P[AB], X, Ym1, Z), Grad3(P[BB], Xm1, Ym1, Z), U),
				 V),
			 Lerp(Lerp(Grad3(P[AA + 1], X, Y, Zm1), Grad3(P[BA + 1], Xm1, Y, Zm1), U),
				 Lerp(Grad3(P[AB + 1], X, Ym1, Zm1), Grad3(P[BB + 1], Xm1, Ym1, Zm1), U),
				 V),
			 W
		),
		-1.0f, 1.0f);
}


// Instantiate for linker
template struct UE::Math::TRotator<float>;
template struct UE::Math::TRotator<double>;
template struct UE::Math::TQuat<float>;
template struct UE::Math::TQuat<double>;
template struct UE::Math::TVector<float>;
template struct UE::Math::TVector<double>;
template struct UE::Math::TVector2<float>;
template struct UE::Math::TVector2<double>;
template struct UE::Math::TVector4<float>;
template struct UE::Math::TVector4<double>;
template struct UE::Math::TMatrix<float>;
template struct UE::Math::TMatrix<double>;
template CORE_API void FMath::SegmentDistToSegmentSafe<float>(UE::Math::TVector<float> A1, UE::Math::TVector<float> B1, UE::Math::TVector<float> A2, UE::Math::TVector<float> B2, UE::Math::TVector<float>& OutP1, UE::Math::TVector<float>& OutP2);
template CORE_API void FMath::SegmentDistToSegmentSafe<double>(UE::Math::TVector<double> A1, UE::Math::TVector<double> B1, UE::Math::TVector<double> A2, UE::Math::TVector<double> B2, UE::Math::TVector<double>& OutP1, UE::Math::TVector<double>& OutP2);
template CORE_API UE::Math::TVector<float> FMath::ClosestPointOnLine(const UE::Math::TVector<float>& LineStart, const UE::Math::TVector<float>& LineEnd, const UE::Math::TVector<float>& Point);
template CORE_API UE::Math::TVector<double> FMath::ClosestPointOnLine(const UE::Math::TVector<double>& LineStart, const UE::Math::TVector<double>& LineEnd, const UE::Math::TVector<double>& Point);
template CORE_API UE::Math::TVector<float> FMath::ClosestPointOnSegment(const UE::Math::TVector<float>& Point, const UE::Math::TVector<float>& StartPoint, const UE::Math::TVector<float>& EndPoint);
template CORE_API UE::Math::TVector<double> FMath::ClosestPointOnSegment(const UE::Math::TVector<double>& Point, const UE::Math::TVector<double>& StartPoint, const UE::Math::TVector<double>& EndPoint);
template CORE_API UE::Math::TVector2<float> FMath::ClosestPointOnSegment2D(const UE::Math::TVector2<float>& Point, const UE::Math::TVector2<float>& StartPoint, const UE::Math::TVector2<float>& EndPoint);
template CORE_API UE::Math::TVector2<double> FMath::ClosestPointOnSegment2D(const UE::Math::TVector2<double>& Point, const UE::Math::TVector2<double>& StartPoint, const UE::Math::TVector2<double>& EndPoint);
template CORE_API UE::Math::TVector2<float> FMath::Vector2DInterpTo(const UE::Math::TVector2<float>& Current, const UE::Math::TVector2<float>& Target, float DeltaTime, float InterpSpeed);
template CORE_API UE::Math::TVector2<double> FMath::Vector2DInterpTo(const UE::Math::TVector2<double>& Current, const UE::Math::TVector2<double>& Target, float DeltaTime, float InterpSpeed);
