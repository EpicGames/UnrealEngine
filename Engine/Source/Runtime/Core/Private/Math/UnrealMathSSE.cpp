// Copyright Epic Games, Inc. All Rights Reserved.

// This file implements non-inlined functions from UnrealMathSSE.h
#include "Math/UnrealMathSSE.h"
#if PLATFORM_ENABLE_VECTORINTRINSICS && !PLATFORM_ENABLE_VECTORINTRINSICS_NEON

#include "Math/Matrix.h"
#define USE_SSE2
#include "ThirdParty/SSEMathFun/sse_mathfun_extension.h"
#undef USE_SSE2

namespace SSE 
{

CORE_API __m128 log_ps(__m128 x)
{
	return ::log_ps(x);
}

CORE_API __m128 exp_ps(__m128 x)
{
	return ::exp_ps(x);
}

CORE_API __m128 sin_ps(__m128 x)
{
	return ::sin_ps(x);
}

CORE_API __m128 cos_ps(__m128 x)
{
	return ::cos_ps(x);
}

CORE_API void sincos_ps(__m128 x, __m128 *s, __m128 *c)
{
	return ::sincos_ps(x, s, c);
}

CORE_API __m128 tan_ps( __m128 x )
{
	return ::tancot_ps( x, 0 );
}

CORE_API __m128 cot_ps(__m128 x)
{
	return ::tancot_ps( x, 1 );
}

CORE_API __m128 atan_ps(__m128 x)
{
	return ::atan_ps(x);
}

CORE_API __m128 atan2_ps(__m128 y, __m128 x )
{
	return ::atan2_ps(y, x);
}

}

void VectorMatrixMultiply(FMatrix44f* Result, const FMatrix44f* Matrix1, const FMatrix44f* Matrix2)
{
	const VectorRegister4Float* A = (const VectorRegister4Float*)Matrix1;
	const VectorRegister4Float* B = (const VectorRegister4Float*)Matrix2;
	VectorRegister4Float* R = (VectorRegister4Float*)Result;
	VectorRegister4Float Temp, R0, R1, R2;

	// First row of result (Matrix1[0] * Matrix2).
	Temp = VectorMultiply(VectorReplicate(A[0], 0), B[0]);
	Temp = VectorMultiplyAdd(VectorReplicate(A[0], 1), B[1], Temp);
	Temp = VectorMultiplyAdd(VectorReplicate(A[0], 2), B[2], Temp);
	R0 = VectorMultiplyAdd(VectorReplicate(A[0], 3), B[3], Temp);

	// Second row of result (Matrix1[1] * Matrix2).
	Temp = VectorMultiply(VectorReplicate(A[1], 0), B[0]);
	Temp = VectorMultiplyAdd(VectorReplicate(A[1], 1), B[1], Temp);
	Temp = VectorMultiplyAdd(VectorReplicate(A[1], 2), B[2], Temp);
	R1 = VectorMultiplyAdd(VectorReplicate(A[1], 3), B[3], Temp);

	// Third row of result (Matrix1[2] * Matrix2).
	Temp = VectorMultiply(VectorReplicate(A[2], 0), B[0]);
	Temp = VectorMultiplyAdd(VectorReplicate(A[2], 1), B[1], Temp);
	Temp = VectorMultiplyAdd(VectorReplicate(A[2], 2), B[2], Temp);
	R2 = VectorMultiplyAdd(VectorReplicate(A[2], 3), B[3], Temp);

	// Fourth row of result (Matrix1[3] * Matrix2).
	Temp = VectorMultiply(VectorReplicate(A[3], 0), B[0]);
	Temp = VectorMultiplyAdd(VectorReplicate(A[3], 1), B[1], Temp);
	Temp = VectorMultiplyAdd(VectorReplicate(A[3], 2), B[2], Temp);
	Temp = VectorMultiplyAdd(VectorReplicate(A[3], 3), B[3], Temp);

	// Store result. Must not be done during steps above in case source and destination are the same.
	R[0] = R0;
	R[1] = R1;
	R[2] = R2;
	R[3] = Temp;
}


void VectorMatrixMultiply(FMatrix44d* Result, const FMatrix44d* Matrix1, const FMatrix44d* Matrix2)
{
	// Warning: FMatrix44d alignment may not match VectorRegister4Double, so you can't just cast to VectorRegister4Double*.
	typedef double Double4x4[4][4];
	const Double4x4& AMRows = *((const Double4x4*)Matrix1->M);
	const Double4x4& BMRows = *((const Double4x4*)Matrix2->M);
	VectorRegister4Double R0, R1, R2, R3;

	VectorRegister4Double B[4];
	B[0] = VectorLoad(BMRows[0]);
	B[1] = VectorLoad(BMRows[1]);
	B[2] = VectorLoad(BMRows[2]);
	B[3] = VectorLoad(BMRows[3]);

	// First row of result (Matrix1[0] * Matrix2).
	{
		const VectorRegister4Double ARow = VectorLoad(AMRows[0]);
		R0 = VectorMultiply(VectorReplicate(ARow, 0), B[0]);
		R0 = VectorMultiplyAdd(VectorReplicate(ARow, 1), B[1], R0);
		R0 = VectorMultiplyAdd(VectorReplicate(ARow, 2), B[2], R0);
		R0 = VectorMultiplyAdd(VectorReplicate(ARow, 3), B[3], R0);
	}

	// Second row of result (Matrix1[1] * Matrix2).
	{
		const VectorRegister4Double ARow = VectorLoad(AMRows[1]);
		R1 = VectorMultiply(VectorReplicate(ARow, 0), B[0]);
		R1 = VectorMultiplyAdd(VectorReplicate(ARow, 1), B[1], R1);
		R1 = VectorMultiplyAdd(VectorReplicate(ARow, 2), B[2], R1);
		R1 = VectorMultiplyAdd(VectorReplicate(ARow, 3), B[3], R1);
	}

	// Third row of result (Matrix1[2] * Matrix2).
	{
		const VectorRegister4Double ARow = VectorLoad(AMRows[2]);
		R2 = VectorMultiply(VectorReplicate(ARow, 0), B[0]);
		R2 = VectorMultiplyAdd(VectorReplicate(ARow, 1), B[1], R2);
		R2 = VectorMultiplyAdd(VectorReplicate(ARow, 2), B[2], R2);
		R2 = VectorMultiplyAdd(VectorReplicate(ARow, 3), B[3], R2);
	}

	// Fourth row of result (Matrix1[3] * Matrix2).
	{
		const VectorRegister4Double ARow = VectorLoad(AMRows[3]);
		R3 = VectorMultiply(VectorReplicate(ARow, 0), B[0]);
		R3 = VectorMultiplyAdd(VectorReplicate(ARow, 1), B[1], R3);
		R3 = VectorMultiplyAdd(VectorReplicate(ARow, 2), B[2], R3);
		R3 = VectorMultiplyAdd(VectorReplicate(ARow, 3), B[3], R3);
	}

	// Store result. Must not be done during steps above in case source and destination are the same.
	Double4x4& ResultDst = *((Double4x4*)Result->M);
	VectorStore(R0, ResultDst[0]);
	VectorStore(R1, ResultDst[1]);
	VectorStore(R2, ResultDst[2]);
	VectorStore(R3, ResultDst[3]);
}


#endif // UE_USING_UNREALMATH_SSE
