/*
Copyright 2019 PureDev Software Limited

Permission to use, copy, modify, and/or distribute this software for any
purpose with or without fee is hereby granted, provided that the above
copyright notice and this permission notice appear in all copies.

THE SOFTWARE IS PROVIDED "AS IS" AND THE AUTHOR DISCLAIMS ALL WARRANTIES
WITH REGARD TO THIS SOFTWARE INCLUDING ALL IMPLIED WARRANTIES OF
MERCHANTABILITY AND FITNESS. IN NO EVENT SHALL THE AUTHOR BE LIABLE FOR
ANY SPECIAL, DIRECT, INDIRECT, OR CONSEQUENTIAL DAMAGES OR ANY DAMAGES
WHATSOEVER RESULTING FROM LOSS OF USE, DATA OR PROFITS, WHETHER IN AN
ACTION OF CONTRACT, NEGLIGENCE OR OTHER TORTIOUS ACTION, ARISING OUT OF
OR IN CONNECTION WITH THE USE OR PERFORMANCE OF THIS SOFTWARE.
*/


#if defined(__UNREAL__)
	#include "MemPro/MemPro.h"
#else
	#include "MemPro.h"
#endif


//------------------------------------------------------------------------
//
// MemProLib.h
//
#ifdef __UNREAL__
	#include "HAL/LowLevelMemTracker.h"
	#include "HAL/PlatformStackWalk.h"	//@EPIC: new header
	#include "HAL/FileManager.h"		//@EPIC: new header
	#include "Misc/Paths.h"				//@EPIC: new header
#endif

//------------------------------------------------------------------------
#if MEMPRO_ENABLED

//------------------------------------------------------------------------
#define MEMPRO_STATIC_ASSERT(expr) typedef char STATIC_ASSERT_TEST[ (expr) ]

//------------------------------------------------------------------------
#include <memory.h>

//------------------------------------------------------------------------
namespace MemPro
{
	//------------------------------------------------------------------------
	namespace Platform
	{
		void* Alloc(int size);
		void Free(void* p, int size);
	}

	//------------------------------------------------------------------------
	class Allocator
	{
	public:
		static void* Alloc(int size)
		{
			return Platform::Alloc(size);
		}

		static void Free(void* p, int size)
		{
			Platform::Free(p, size);
		}
	};
}

//------------------------------------------------------------------------
#endif		// #if MEMPRO_ENABLED

//------------------------------------------------------------------------
//
// CallstackSet.h
//
//------------------------------------------------------------------------
#if MEMPRO_ENABLED

//------------------------------------------------------------------------
namespace MemPro
{
	//------------------------------------------------------------------------
	struct Callstack
	{
		uint64* mp_Stack;
		int m_ID;
		int m_Size;
		unsigned int m_Hash;
	};

	//------------------------------------------------------------------------
	// A hash set collection for Callstack structures. Callstacks are added and
	// retreived using the stack address array as the key.
	// This class only allocates memory using virtual alloc/free to avoid going
	// back into the mian allocator.
	class CallstackSet
	{
	public:
		CallstackSet();

		~CallstackSet();

		Callstack* Get(uint64* p_stack, int stack_size, unsigned int hash);

		Callstack* Add(uint64* p_stack, int stack_size, unsigned int hash);

		void Clear();

	private:
		void Grow();

		void Add(Callstack* p_callstack);

		//------------------------------------------------------------------------
		// data
	private:
		Callstack** mp_Data;
		unsigned int m_CapacityMask;
		int m_Count;
		int m_Capacity;
	};
}

//------------------------------------------------------------------------
#endif		// #if MEMPRO_ENABLED

//------------------------------------------------------------------------
//
// BlockAllocator.hpp
//
//------------------------------------------------------------------------
#if MEMPRO_ENABLED

//------------------------------------------------------------------------
// disable some warnings we are not interested in so that we can compile at warning level4
#ifdef MEMPRO_ENABLE_WARNING_PRAGMAS
	MEMPRO_PUSH_WARNING_DISABLE
	MEMPRO_DISABLE_WARNING(4100)
#endif

//------------------------------------------------------------------------
namespace MemPro
{
	//------------------------------------------------------------------------
	// a very simple allocator tat allocated blocks of 64k of memory using the
	// templatized allocator.
	template<class TAllocator>
	class BlockAllocator
	{
	public:
		inline BlockAllocator();

		inline void* Alloc(int size);

		inline void Free(void* p);

		//------------------------------------------------------------------------
		// data
	private:
		static const int m_BlockSize = 1024*1024;
		void* mp_CurBlock;
		int m_CurBlockUsage;
	};

	//------------------------------------------------------------------------
	template<class TAllocator>
	BlockAllocator<TAllocator>::BlockAllocator()
	:	mp_CurBlock(NULL),
		m_CurBlockUsage(0)
	{
	}

	//------------------------------------------------------------------------
	template<class TAllocator>
	void* BlockAllocator<TAllocator>::Alloc(int size)
	{
		MEMPRO_ASSERT(size < m_BlockSize);

		if(!mp_CurBlock || size > m_BlockSize - m_CurBlockUsage)
		{
			mp_CurBlock = TAllocator::Alloc(m_BlockSize);
			MEMPRO_ASSERT(mp_CurBlock);
			m_CurBlockUsage = 0;
		}

		void* p = (char*)mp_CurBlock + m_CurBlockUsage;
		m_CurBlockUsage += size;

		return p;
	}

	//------------------------------------------------------------------------
	template<class TAllocator>
	void BlockAllocator<TAllocator>::Free(void* p)
	{
		// do nothing
	}
}

//------------------------------------------------------------------------
#ifdef MEMPRO_ENABLE_WARNING_PRAGMAS
	MEMPRO_POP_WARNING_DISABLE
#endif

//------------------------------------------------------------------------
#endif		// #if MEMPRO_ENABLED

//------------------------------------------------------------------------
//
// CallstackSet.cpp
//
//------------------------------------------------------------------------
#if MEMPRO_ENABLED

//------------------------------------------------------------------------
namespace MemPro
{
	//------------------------------------------------------------------------
	const int g_InitialCapacity = 4096;		// must be a power of 2

	MemPro::BlockAllocator<Allocator> g_BlockAllocator;

	//------------------------------------------------------------------------
	inline bool StacksMatch(MemPro::Callstack* p_callstack, uint64* p_stack, int stack_size, unsigned int hash)
	{
		if(p_callstack->m_Size != stack_size)
			return false;

		if(p_callstack->m_Hash != hash)
			return false;

		for(int i=0; i<stack_size; ++i)
			if(p_callstack->mp_Stack[i] != p_stack[i])
				return false;

		return true;
	}
}

//------------------------------------------------------------------------
MemPro::CallstackSet::CallstackSet()
:	mp_Data((Callstack**)Allocator::Alloc(g_InitialCapacity*sizeof(Callstack*))),
	m_CapacityMask(g_InitialCapacity-1),
	m_Count(0),
	m_Capacity(g_InitialCapacity)
{
	memset(mp_Data, 0, g_InitialCapacity*sizeof(Callstack*));
}

//------------------------------------------------------------------------
MemPro::CallstackSet::~CallstackSet()
{
	Clear();
}

//------------------------------------------------------------------------
void MemPro::CallstackSet::Grow()
{
	int old_capacity = m_Capacity;
	Callstack** p_old_data = mp_Data;

	// allocate a new set
	m_Capacity *= 2;
	m_CapacityMask = m_Capacity - 1;
	int size = m_Capacity * sizeof(Callstack*);
	mp_Data = (Callstack**)Allocator::Alloc(size);
	memset(mp_Data, 0, size);

	// transfer callstacks from old set
	m_Count = 0;
	for(int i=0; i<old_capacity; ++i)
	{
		Callstack* p_callstack = p_old_data[i];
		if(p_callstack)
			Add(p_callstack);
	}

	// release old buffer
	Allocator::Free(p_old_data, old_capacity*sizeof(Callstack*));
}

//------------------------------------------------------------------------
MemPro::Callstack* MemPro::CallstackSet::Get(uint64* p_stack, int stack_size, unsigned int hash)
{
	int index = hash & m_CapacityMask;

	while(mp_Data[index] && !StacksMatch(mp_Data[index], p_stack, stack_size, hash))
		index = (index + 1) & m_CapacityMask;

	return mp_Data[index];
}

//------------------------------------------------------------------------
MemPro::Callstack* MemPro::CallstackSet::Add(uint64* p_stack, int stack_size, unsigned int hash)
{
	// grow the set if necessary
	if(m_Count > m_Capacity/4)
		Grow();

	// create a new callstack
	Callstack* p_callstack = (Callstack*)g_BlockAllocator.Alloc(sizeof(Callstack));
	p_callstack->m_ID = m_Count;
	p_callstack->m_Size = stack_size;
	p_callstack->mp_Stack = (uint64*)g_BlockAllocator.Alloc(stack_size*sizeof(uint64));
	p_callstack->m_Hash = hash;
	Platform::MemCpy(p_callstack->mp_Stack, stack_size*sizeof(uint64), p_stack, stack_size*sizeof(uint64));

	Add(p_callstack);

	return p_callstack;
}

//------------------------------------------------------------------------
void MemPro::CallstackSet::Add(Callstack* p_callstack)
{
	// find a clear index
	int index = p_callstack->m_Hash & m_CapacityMask;
	while(mp_Data[index])
		index = (index + 1) & m_CapacityMask;

	mp_Data[index] = p_callstack;

	++m_Count;
}

//------------------------------------------------------------------------
void MemPro::CallstackSet::Clear()
{
	for(int i=0; i<m_Capacity; ++i)
	{
		if(mp_Data[i])
			g_BlockAllocator.Free(mp_Data[i]);
	}

	Allocator::Free(mp_Data, m_Capacity*sizeof(Callstack*));

	size_t size = g_InitialCapacity*sizeof(Callstack*);
	mp_Data = (Callstack**)Allocator::Alloc((int)size);
	memset(mp_Data, 0, size);
	m_CapacityMask = g_InitialCapacity-1;
	m_Count = 0;
	m_Capacity = g_InitialCapacity;
}

//------------------------------------------------------------------------
#endif		// #if MEMPRO_ENABLED

//------------------------------------------------------------------------
//
// CriticalSection.h
//
//------------------------------------------------------------------------
#if MEMPRO_ENABLED

//------------------------------------------------------------------------
namespace MemPro
{
	//------------------------------------------------------------------------
	class CriticalSection
	{
	public:
		CriticalSection()
		{
			Platform::CreateLock(m_OSLockMem, sizeof(m_OSLockMem));
		}

		~CriticalSection()
		{
			Platform::DestroyLock(m_OSLockMem);
		}

		void Enter()
		{
			Platform::TakeLock(m_OSLockMem);
		}

		void Leave()
		{
			Platform::ReleaseLock(m_OSLockMem);
		}
	private:

		//------------------------------------------------------------------------
		// data
	private:
		static const int m_OSLockMaxSize = 64;
		char m_OSLockMem[m_OSLockMaxSize];
	} MEMPRO_ALIGN_SUFFIX(16);

	//------------------------------------------------------------------------
	class CriticalSectionScope
	{
	public:
		CriticalSectionScope(CriticalSection& in_cs) : cs(in_cs) { cs.Enter(); }
		~CriticalSectionScope() { cs.Leave(); }
	private:
		CriticalSectionScope(const CriticalSectionScope&);
		CriticalSectionScope& operator=(const CriticalSectionScope&);
		CriticalSection& cs;
	};
}

//------------------------------------------------------------------------
#endif		// #if MEMPRO_ENABLED

//------------------------------------------------------------------------
//
// RingBuffer.h
//
//------------------------------------------------------------------------
#if MEMPRO_ENABLED

//------------------------------------------------------------------------

//------------------------------------------------------------------------
//
// Event.h
//
//------------------------------------------------------------------------
#if MEMPRO_ENABLED

//------------------------------------------------------------------------
namespace MemPro
{
	//--------------------------------------------------------------------
	class Event
	{
	public:
		//--------------------------------------------------------------------
		Event(bool initial_state, bool auto_reset)
		{
			Platform::MemProCreateEvent(
				m_OSEventMem,
				m_OSEventMemMaxSize,
				initial_state,
				auto_reset);
		}

		//--------------------------------------------------------------------
		~Event()
		{
			Platform::DestroyEvent(m_OSEventMem);
		}

		//--------------------------------------------------------------------
		void Set() const
		{
			Platform::SetEvent(m_OSEventMem);
		}

		//--------------------------------------------------------------------
		void Reset()
		{
			Platform::SetEvent(m_OSEventMem);
		}

		//--------------------------------------------------------------------
		int Wait(int timeout=-1) const
		{
			return Platform::WaitEvent(m_OSEventMem, timeout);
		}

		//------------------------------------------------------------------------
		// data
	private:
		static const int m_OSEventMemMaxSize = 144;
		mutable char m_OSEventMem[m_OSEventMemMaxSize];
	} MEMPRO_ALIGN_SUFFIX(16);
}

//------------------------------------------------------------------------
#endif		// #if MEMPRO_ENABLED

//------------------------------------------------------------------------
//#define USE_CRITICAL_SECTIONS

//------------------------------------------------------------------------
namespace MemPro
{
	//------------------------------------------------------------------------
	// This ring buffer is a lockless buffer, designed to be accessed by no more
	// than two threads, one thread adding to the buffer and one removing. The
	// threads first request data and then add or remove tat data. The threads
	// will sleep if there is no space to add or no data to remove. Once the
	// threads have space on the buffer the data can be added or removed.
	class RingBuffer
	{
	public:
		//------------------------------------------------------------------------
		struct Range
		{
			Range() {}
			Range(void* p, int s) : mp_Buffer(p), m_Size(s) {}

			void* mp_Buffer;
			int m_Size;
		};

		//------------------------------------------------------------------------
		RingBuffer(char* p_buffer, int size)
		:	m_Size(size),
			mp_Buffer(p_buffer),
			m_UsedRange(0),
			m_BytesRemovedEvent(false, true),
			m_BytesAddedEvent(false, true)
		{
			MEMPRO_ASSERT(IsPow2(size));

#ifdef USE_CRITICAL_SECTIONS
			InitializeCriticalSection(&m_CriticalSection);
#endif
		}

		//------------------------------------------------------------------------
		inline bool IsPow2(int value)
		{
			return (value & (value-1)) == 0;
		}

		//------------------------------------------------------------------------
		int GetSize() const
		{
			return m_Size;
		}

		//------------------------------------------------------------------------
		void Lock() const
		{
#ifdef USE_CRITICAL_SECTIONS
			EnterCriticalSection(&m_CriticalSection);
#endif
		}

		//------------------------------------------------------------------------
		void Release() const
		{
#ifdef USE_CRITICAL_SECTIONS
			LeaveCriticalSection(&m_CriticalSection);
#endif
		}

		//------------------------------------------------------------------------
		int64 GetRangeAtomic() const
		{
#ifdef USE_CRITICAL_SECTIONS
			Lock();
			int64 range = m_UsedRange;
			Release();
#else
			// there must be a better way to atomically read a 64 bit value.
			int64 range = Platform::MemProInterlockedExchangeAdd(const_cast<int64*>(&m_UsedRange), 0);
#endif
			return range;
		}

		//------------------------------------------------------------------------
		// return the largest free range possible
		Range GetFreeRange(int timeout=-1) const
		{
			int64 range = GetRangeAtomic();
			int size = (int)(range & 0xffffffff);

			// wait until there is some space
			while(size == m_Size)
			{
				if(!m_BytesRemovedEvent.Wait(timeout))
					return Range(NULL, 0);

				range = GetRangeAtomic();
				size = (int)(range & 0xffffffff);
			}

			int start = (int)((range >> 32) & 0xffffffff);

			// calculate the size
			int free_start = (start + size) & (m_Size-1);
			int free_size = free_start < start ? start - free_start : m_Size - free_start;

			return Range(mp_Buffer + free_start, free_size);
		}

		//------------------------------------------------------------------------
		// return the largest used range
		Range GetAllocatedRange(int timeout=-1) const
		{
			int64 range = GetRangeAtomic();

			MEMPRO_INSTRUCTION_BARRIER;

			int size = (int)(range & 0xffffffff);

			// wait until there is some data
			while(!size)
			{
				if(!m_BytesAddedEvent.Wait(timeout))
					return Range(NULL, 0);

				range = GetRangeAtomic();
				size = (int)(range & 0xffffffff);
			}

			int start = (int)((range >> 32) & 0xffffffff);

			// calculate the size
			int max_size = m_Size - start;
			if(size > max_size)
				size = max_size;

			return Range(mp_Buffer + start, size);
		}

		//------------------------------------------------------------------------
		// tells the ring buffer how many bytes have been copied to the allocated range
		void Add(int size)
		{
			Lock();

			MEMPRO_ASSERT(size >= 0);

			volatile int64 old_range;
			int64 new_range;

			do
			{
				old_range = GetRangeAtomic();
				
				int64 used_size = (old_range) & 0xffffffff;
				used_size += size;
				new_range = (old_range & 0xffffffff00000000LL) | used_size;

			} while(Platform::MemProInterlockedCompareExchange(&m_UsedRange, new_range, old_range) != old_range); //-V567

			m_BytesAddedEvent.Set();

			Release();
		}

		//------------------------------------------------------------------------
		// tells the ring buffer how many bytes have been removed from the allocated range
		void Remove(int size)
		{
			Lock();

			MEMPRO_ASSERT(size >= 0);

			volatile int64 old_range;
			int64 new_range;
			int mask = m_Size - 1;

			do
			{
				old_range = GetRangeAtomic();
				
				int64 used_start = (old_range >> 32) & 0xffffffff;
				int64 used_size = (old_range) & 0xffffffff;
				used_start = (used_start + size) & mask;
				used_size -= size;
				new_range = (used_start << 32) | used_size;

			} while(Platform::MemProInterlockedCompareExchange(&m_UsedRange, new_range, old_range) != old_range); //-V567

			m_BytesRemovedEvent.Set();

			Release();
		}

		//------------------------------------------------------------------------
		int GetUsedBytes() const
		{
			return (int)(m_UsedRange & 0xffffffff);
		}

		//------------------------------------------------------------------------
		void Clear()
		{
			m_UsedRange = 0;
			m_BytesRemovedEvent.Reset();
			m_BytesAddedEvent.Reset();
		}

		//------------------------------------------------------------------------
		// data
	private:
		int m_Size;
		char* mp_Buffer;

		MEMPRO_INTERLOCKED_ALIGN int64 m_UsedRange;		// start index is the high int, size is the low int

#ifdef USE_CRITICAL_SECTIONS
		mutable CRITICAL_SECTION m_CriticalSection;
#endif
		Event m_BytesRemovedEvent;
		Event m_BytesAddedEvent;
	};
}

//------------------------------------------------------------------------
#endif		// #if MEMPRO_ENABLED

//------------------------------------------------------------------------
//
// MemProMisc.h
//
#include <stdlib.h>


//------------------------------------------------------------------------
#if MEMPRO_ENABLED

//------------------------------------------------------------------------
// disable some warnings we are not interested in so that we can compile at warning level4
#ifdef MEMPRO_ENABLE_WARNING_PRAGMAS
	MEMPRO_PUSH_WARNING_DISABLE
	MEMPRO_DISABLE_WARNING(4127)
#endif

//------------------------------------------------------------------------
#define MEMPRO_SPINLOCK_FREE_VAL 0
#define MEMPRO_SPINLOCK_LOCKED_VAL 1
#define MEMPRO_YIELD_SPIN_COUNT 40
#define MEMPRO_SLEEP_SPIN_COUNT 200

//------------------------------------------------------------------------
namespace MemPro
{
	//------------------------------------------------------------------------
	inline int Min(int a, int b) { return a < b ? a : b; }

	//------------------------------------------------------------------------
	inline void SwapEndian(unsigned int& value)
	{
		Platform::SwapEndian(value);
	}

	//------------------------------------------------------------------------
	inline void SwapEndian(uint64& value)
	{
		Platform::SwapEndian(value);
	}

	//------------------------------------------------------------------------
	inline void SwapEndian(int64& value)
	{
		SwapEndian((uint64&)value);
	}

	//------------------------------------------------------------------------
	template<typename T>
	inline void SwapEndian(T& value)
	{
		MEMPRO_ASSERT(sizeof(T) == sizeof(unsigned int));
		SwapEndian((unsigned int&)value);
	}

	//------------------------------------------------------------------------
	inline void SwapEndianUInt64Array(void* p, int size)
	{
		MEMPRO_ASSERT(size % 8 == 0);
		uint64* p_uint64 = (uint64*)p;
		uint64* p_end = p_uint64 + size/8;
		while(p_uint64 != p_end)
			SwapEndian(*p_uint64++);
	}

	//------------------------------------------------------------------------
	inline int64 GetTime()
	{
		return Platform::GetHiResTimer();
	}

	//------------------------------------------------------------------------
	inline int64 GetTickFrequency()
	{
		return Platform::GetHiResTimerFrequency();
	}

	//------------------------------------------------------------------------
	inline void SetThreadName(unsigned int thread_id, const char* p_name)
	{
		Platform::SetThreadName(thread_id, p_name);
	}

	//------------------------------------------------------------------------
	inline void SmallFastMemCpy(void* p_dst, void* p_src, int size)
	{
		MEMPRO_ASSERT((((size_t)p_dst) & 3) == 0);
		MEMPRO_ASSERT((((size_t)p_src) & 3) == 0);
		MEMPRO_ASSERT((size & 3) == 0);

		unsigned int uint_count = size / sizeof(unsigned int);
		unsigned int* p_uint_dst = (unsigned int*)p_dst;
		unsigned int* p_uint_src = (unsigned int*)p_src;
		for(unsigned int i=0; i<uint_count; ++i)
			*p_uint_dst++ = *p_uint_src++;
	}
}

//------------------------------------------------------------------------
#ifdef MEMPRO_ENABLE_WARNING_PRAGMAS
	MEMPRO_POP_WARNING_DISABLE
#endif

//------------------------------------------------------------------------
#endif		// #if MEMPRO_ENABLED

//------------------------------------------------------------------------
//
// Packets.h
//
//------------------------------------------------------------------------
#if MEMPRO_ENABLED

//------------------------------------------------------------------------
namespace MemPro
{
	//------------------------------------------------------------------------
	// This file contains all of te packets that can be sent to the MemPro app.

	//------------------------------------------------------------------------
	enum PacketType
	{
		EInvalid = 0xabcd,
		EAllocPacket,
		EFreePacket,
		ECallstackPacket,
		EPageStatePacket,
		EPageStateStartPacket,	// for backwards compatibility
		EPageStateEndPacket_OLD,
		EVirtualMemStats,
		ETakeSnapshot,
		EVMemStats,
		EPageStateEndPacket,
		EDataStoreEndPacket,
		EPulsePacket,
		ERequestShutdown
	};

	//------------------------------------------------------------------------
	enum MemProVersion
	{
		Version = 14
	};

	//------------------------------------------------------------------------
	enum MemProClientFlags
	{
		SendPageData = 0,
		SendPageDataWithMemory,
		EShutdownComplete
	};

	//------------------------------------------------------------------------
	// value that is sent immediatley after connection to detect big endian
	enum EEndianKey
	{
		EndianKey = 0xabcdef01
	};

	//------------------------------------------------------------------------
	inline uint64 ObfuscateAddress(uint64 addr)
	{
		const uint64 mask = 0x12345678abcdef12LL;

		return addr ^ mask;
	}

	//------------------------------------------------------------------------
	inline uint64 UnobfuscateAddress(uint64 addr)
	{
		return ObfuscateAddress(addr);
	}

	//------------------------------------------------------------------------
	struct PacketHeader
	{
		PacketType m_PacketType;
		int m_Padding;
		int64 m_Time;

		void SwapEndian()
		{
			MemPro::SwapEndian(m_PacketType);
			MemPro::SwapEndian(m_Time);
		}
	};

	//------------------------------------------------------------------------
	struct ConnectPacket
	{
		uint64 m_Padding;			// for backwards compatibility

		int64 m_ConnectTime;
		int64 m_TickFrequency;

		int m_Version;
		int m_PtrSize;

		EPlatform m_Platform;
		int m_Padding2;

		void SwapEndian()
		{
			MemPro::SwapEndian(m_Version);
			MemPro::SwapEndian(m_ConnectTime);
			MemPro::SwapEndian(m_TickFrequency);
			MemPro::SwapEndian(m_PtrSize);
		}
	};

	//------------------------------------------------------------------------
	struct AllocPacket
	{
		uint64 m_Addr;
		uint64 m_Size;
		int m_CallstackID;
		int m_Padding;

		void SwapEndian()
		{
			MemPro::SwapEndian(m_Addr);
			MemPro::SwapEndian(m_Size);
			MemPro::SwapEndian(m_CallstackID);
		}
	};

	//------------------------------------------------------------------------
	struct FreePacket
	{
		uint64 m_Addr;

		void SwapEndian()
		{
			MemPro::SwapEndian(m_Addr);
		}
	};

	//------------------------------------------------------------------------
	struct PageStatePacket
	{
		uint64 m_Addr;
		uint64 m_Size;
		PageState m_State;
		PageType m_Type;
		unsigned int m_Protection;
		int m_SendingMemory;

		void SwapEndian()
		{
			MemPro::SwapEndian(m_Addr);
			MemPro::SwapEndian(m_Size);
			MemPro::SwapEndian(m_State);
			MemPro::SwapEndian(m_Type);
			MemPro::SwapEndian(m_Protection);
			MemPro::SwapEndian(m_SendingMemory);
		}
	};

	//------------------------------------------------------------------------
	struct VirtualMemStatsPacket
	{
		uint64 m_Reserved;
		uint64 m_Committed;

		void SwapEndian()
		{
			MemPro::SwapEndian(m_Reserved);
			MemPro::SwapEndian(m_Committed);
		}
	};

	//------------------------------------------------------------------------
	struct IgnoreMemRangePacket
	{
		uint64 m_Addr;
		uint64 m_Size;

		void SwapEndian()
		{
			MemPro::SwapEndian(m_Addr);
			MemPro::SwapEndian(m_Size);
		}
	};

	//------------------------------------------------------------------------
	struct TakeSnapshotPacket
	{
		int m_IsMemorySnapshot;

		void SwapEndian()
		{
			MemPro::SwapEndian(m_IsMemorySnapshot);
		}
	};
}

//------------------------------------------------------------------------
#endif		// #if MEMPRO_ENABLED

//------------------------------------------------------------------------
//
// Socket.h
//
//------------------------------------------------------------------------
#if MEMPRO_ENABLED && !defined(MEMPRO_WRITE_DUMP)

//------------------------------------------------------------------------
namespace MemPro
{
	//------------------------------------------------------------------------
	class SocketImp;

	//------------------------------------------------------------------------
	class Socket
	{
	public:
		inline Socket();

		void Disconnect();

		bool Bind(const char* p_port);

		bool StartListening();

		bool Accept(Socket& client_socket);

		int Receive(void* p_buffer, int size);

		bool Send(void* p_buffer, int size);

		inline bool IsValid() const { return Platform::IsValidSocket(m_OSSocketMem); }

		//------------------------------------------------------------------------
		// data
		static const int m_OSSocketMemMaxSize = 8;
		char m_OSSocketMem[m_OSSocketMemMaxSize];
	} MEMPRO_ALIGN_SUFFIX(16);

	//------------------------------------------------------------------------
	Socket::Socket()
	{
		Platform::CreateSocket(m_OSSocketMem, sizeof(m_OSSocketMem));
	}
}

//------------------------------------------------------------------------
#endif		// #if MEMPRO_ENABLED && !defined(MEMPRO_WRITE_DUMP)

//------------------------------------------------------------------------
//
// Thread.h
//
#if MEMPRO_ENABLED

//------------------------------------------------------------------------

//------------------------------------------------------------------------
namespace MemPro
{
	//------------------------------------------------------------------------
	class Thread
	{
	public:
		Thread();

		~Thread();

		int CreateThread(ThreadMain p_thread_main, void* p_param=NULL);

		bool IsAlive() const { return Platform::IsThreadAlive(m_OSThread); }

		//------------------------------------------------------------------------
		// data
	private:
		static const int m_OSThreadMaxSize = 32;
		char m_OSThread[m_OSThreadMaxSize];
	} MEMPRO_ALIGN_SUFFIX(16);
}

//------------------------------------------------------------------------
#endif		// #if MEMPRO_ENABLED

//------------------------------------------------------------------------
//
// MemProFile.h
//
//------------------------------------------------------------------------
#if MEMPRO_ENABLED

//------------------------------------------------------------------------
namespace MemPro
{
	//--------------------------------------------------------------------
	class File
	{
	public:
		//--------------------------------------------------------------------
		File()
		:	m_Opened(false)
		{
			Platform::MemProCreateFile(m_OSFileMem, m_OSFileMemMaxSize);
		}

		//--------------------------------------------------------------------
		~File()
		{
			Platform::DestroyFile(m_OSFileMem);
		}

		//--------------------------------------------------------------------
		bool OpenForWrite(const char* p_filename)
		{
			MEMPRO_ASSERT(!m_Opened);

			m_Opened = Platform::OpenFileForWrite(m_OSFileMem, p_filename);

			return m_Opened;
		}

		//------------------------------------------------------------------------
		void Close()
		{
			MEMPRO_ASSERT(m_Opened);

			Platform::CloseFile(m_OSFileMem);

			m_Opened = false;
		}

		//------------------------------------------------------------------------
		void Flush()
		{
			MEMPRO_ASSERT(m_Opened);

			Platform::FlushFile(m_OSFileMem);
		}

		//------------------------------------------------------------------------
		bool Write(const void* p_data, int size)
		{
			MEMPRO_ASSERT(m_Opened);

			return Platform::WriteFile(m_OSFileMem, p_data, size);
		}

		//------------------------------------------------------------------------
		// data
	private:
		static const int m_OSFileMemMaxSize = 16;
		mutable char m_OSFileMem[m_OSFileMemMaxSize];

		bool m_Opened;
	};
}

//------------------------------------------------------------------------
#endif		// #if MEMPRO_ENABLED

//------------------------------------------------------------------------
//
// MemPro.cpp
//
#include <new>
#include <stdio.h>
#include <time.h>
#include <limits.h>
#include <string.h>

//------------------------------------------------------------------------
#if MEMPRO_ENABLED

//------------------------------------------------------------------------
// disable some warnings we are not interested in so that we can compile at warning level4
#ifdef MEMPRO_ENABLE_WARNING_PRAGMAS
	MEMPRO_PUSH_WARNING_DISABLE
	MEMPRO_DISABLE_WARNING(4127)
	MEMPRO_DISABLE_WARNING(4100)
#endif

//------------------------------------------------------------------------
#ifdef VMEM_ENABLE_STATS
namespace VMem { void SendStatsToMemPro(void (*send_fn)(void*, int, void*), void* p_context); }
#endif

//------------------------------------------------------------------------
//#define TEST_ENDIAN

//#define PACKET_START_END_MARKERS

#ifdef TEST_ENDIAN
	#define ENDIAN_TEST(a) a
#else
	#define ENDIAN_TEST(a)
#endif

//------------------------------------------------------------------------
namespace MemPro
{
	THREAD_LOCAL_STORAGE void** g_CallstackDataTLS = NULL;
}

//------------------------------------------------------------------------
namespace MemPro
{
	//------------------------------------------------------------------------
	int g_MemProRefs = 0;

	//------------------------------------------------------------------------
	void InitialiseInternal();

	//------------------------------------------------------------------------
	// MemPro will initialise on the first allocation, but this global ensures
	// that MemPro is initialised in the main module. This is sometimes necessary
	// if the first allocation comes from a dll.
	/*
	class Initialiser
	{
	public:
		Initialiser() { MemPro::Initialise(WAIT_FOR_CONNECT); }
	} g_Initialiser;
	*/

	//------------------------------------------------------------------------
	// port number
	const char* g_DefaultPort = MEMPRO_PORT;

	//------------------------------------------------------------------------
	// globals
	const int g_RingBufferSize = 32*1024;

#ifdef MEMPRO64
	uint64 g_MaxAddr = ULLONG_MAX;
#else
	uint64 g_MaxAddr = UINT_MAX;
#endif

	//------------------------------------------------------------------------
#ifdef MEMPRO_WRITE_DUMP
	File g_DumpFile;
#endif

	//------------------------------------------------------------------------
	uint64 ToUInt64(void* p)
	{
#ifdef MEMPRO64
		return (uint64)p;
#else
		unsigned int u = (unsigned int)p;	// cast to uint first to avoid signed bit in casting
		return (uint64)u;
#endif
	}

	//------------------------------------------------------------------------
	struct DataStorePageHeader
	{
		int m_Size;
		DataStorePageHeader* mp_Next;
	};

	//------------------------------------------------------------------------
	struct CallstackCapture
	{
		void** mp_Stack;
		int m_Size;
		unsigned int m_Hash;
	};

	//------------------------------------------------------------------------
	void BaseAddressLookupFunction()
	{
	}

	//------------------------------------------------------------------------
	class CMemPro
	{
	public:
		CMemPro();

		bool Initialise();

		void Shutdown();

		void Disconnect(bool listen_for_new_connection);

		void TrackAlloc(void* p, size_t size, bool wait_for_connect);

		void TrackFree(void* p, bool wait_for_connect);

		static void StaticSendPageState(void* p, size_t size, PageState page_state, PageType page_type, unsigned int page_protection, bool send_page_mem, int page_size, void* p_context);

		void SendPageState(void* p, size_t size, PageState page_state, PageType page_type, unsigned int page_protection, bool send_page_mem, int page_size);

		void TakeSnapshot(bool send_memory);

		void FlushDumpFile();

		int SendThreadMain(void* p_param);

#ifndef MEMPRO_WRITE_DUMP
		int ReceiveThreadMain(void* p_param);
#endif

		int WaitForConnectionThreadMain(void* p_param);

		void Lock() { m_CriticalSection.Enter(); }

		void Release() { m_CriticalSection.Leave(); }

		void WaitForConnectionOnInitialise();

		bool IsPaused();

		void SetPaused(bool paused);

	private:
		static void GetStackTrace(void** stack, int& stack_size, unsigned int& hash);

		void SendModuleInfo();

		void SendExtraModuleInfo(int64 ModuleBase);

		void SendString(const char* p_str);

		static void StaticEnumerateLoadedModulesCallback(int64 module_base, const char* p_module_name, void* p_context);
		
		void EnumerateLoadedModulesCallback(int64 module_base, const char* p_module_name);

		void StoreData(const void* p_data, int size);

		void BlockUntilSendThreadEmpty();

		void SendStoredData();

		void ClearStoreData();

		inline bool SendThreadStillAlive() const;

		void FlushRingBufferForShutdown();

		void SendData(const void* p_data, int size);

		bool SocketSendData(const void* p_data, int size);

		static void StaticSendVMemStatsData(void* p_data, int size, void* p_context);

		void SendVMemStatsData(void* p_data, int size);

		void SendData(unsigned int value);

		inline void SendPacketHeader(PacketType value);

		void SendStartMarker();

		void SendEndMarker();

		inline void Send(bool value);

		template<typename T> void Send(T& value) { SendData(&value, sizeof(value)); }

		void Send(unsigned int value) { SendData(value); }

		void SendPageState(bool send_memory);

		void SendVMemStats();

		void SendVirtualMemStats();

		void** AllocateStackTraceData();

		CallstackCapture CaptureCallstack();

		int SendCallstack(const CallstackCapture& callstack_capture);

		bool WaitForConnection();

		bool WaitForConnectionIfListening();

		static int SendThreadMainStatic(void* p_param);

		static int ReceiveThreadMainStatic(void* p_param);

		static int WaitForConnectionThreadMainStatic(void* p_param);

		static int PulseThreadMainStatic(void* p_param);

		void PulseThreadMain();

		void BlockUntilReadyToSend();

		//------------------------------------------------------------------------
		// data
#ifndef MEMPRO_WRITE_DUMP
		Socket m_ListenSocket;
		Socket m_ClientSocket;
#endif

		CallstackSet m_CallstackSet;

		RingBuffer m_RingBuffer;
		char m_RingBufferMem[g_RingBufferSize];

		volatile bool m_Connected;
		volatile bool m_ReadyToSend;

		volatile bool m_InEvent;

		volatile bool m_Paused;

		Event m_StartedListeningEvent;
		Event m_WaitForConnectThreadFinishedEvent;
		Event m_SendThreadFinishedEvent;
		Event m_ReceiveThreadFinishedEvent;
		Event m_MemProReadyToShutdownEvent;
		Event m_PulseThreadFinished;

		volatile bool m_StartedListening;
		volatile bool m_InitialConnectionTimedOut;

		// only used if MEMPRO_WRITE_DUMP isn't defined and is a win based platform
		int m_LastPageStateSend;
		int m_PageStateInterval;
		int m_LastVMemStatsSend;
		int m_VMemStatsSendInterval;

		bool m_WaitForConnect;

		static const int m_DataStorePageSize = 4096;
		DataStorePageHeader* mp_DataStoreHead;		// used to store allocs before initialised
		DataStorePageHeader* mp_DataStoreTail;

		Thread m_SendThread;
		Thread m_ReceiveThread;
		Thread m_PulseThread;
		Thread m_WaitForConnectionThread;

		bool m_FlushedRingBufferForShutdown;

		CriticalSection m_CriticalSection;
		CriticalSection m_DisconnectCriticalSection;

		int m_ModulesSent;

		volatile bool m_ShuttingDown;

		BlockAllocator<Allocator> m_BlockAllocator;
	};

	//------------------------------------------------------------------------
	char g_MemProMem[sizeof(CMemPro)] MEMPRO_ALIGN_SUFFIX(16);
	CMemPro* gp_MemPro = NULL;
	volatile bool g_ShuttingDown = false;

	//------------------------------------------------------------------------
	inline CMemPro* GetMemPro()
	{
		if(!gp_MemPro)
			InitialiseInternal();

		return gp_MemPro;
	}

	//------------------------------------------------------------------------
	CMemPro::CMemPro()
	:	m_RingBuffer(m_RingBufferMem, g_RingBufferSize), //-V670
		m_Connected(false),
		m_ReadyToSend(false),
		m_InEvent(false),
		m_Paused(false),
		m_StartedListeningEvent(false, false),
		m_WaitForConnectThreadFinishedEvent(false, false),
		m_SendThreadFinishedEvent(true, false),
		m_ReceiveThreadFinishedEvent(true, false),
		m_MemProReadyToShutdownEvent(false, false),
		m_PulseThreadFinished(true, false),
		m_StartedListening(false),
		m_InitialConnectionTimedOut(false),
		m_LastPageStateSend(0),
		m_PageStateInterval(1000),
		m_LastVMemStatsSend(0),
		m_VMemStatsSendInterval(5000),
		m_WaitForConnect(false),
		mp_DataStoreHead(NULL),
		mp_DataStoreTail(NULL),
		m_FlushedRingBufferForShutdown(false),
		m_ModulesSent(0),
		m_ShuttingDown(false)
	{
	}

	//------------------------------------------------------------------------
	void CMemPro::GetStackTrace(void** stack, int& stack_size, unsigned int& hash)
	{
		Platform::GetStackTrace(stack, stack_size, hash);
	}

	//------------------------------------------------------------------------
	void CMemPro::StaticSendVMemStatsData(void* p_data, int size, void* p_context)
	{
		CMemPro* p_this = (CMemPro*)p_context;
		p_this->SendVMemStatsData(p_data, size);
	}

	//------------------------------------------------------------------------
	void CMemPro::SendVMemStatsData(void* p_data, int size)
	{
		static char buffer[256];
		MEMPRO_ASSERT(size <= (int)sizeof(buffer));
		Platform::MemCpy(buffer, sizeof(buffer), p_data, size);
		ENDIAN_TEST(SwapEndianUInt64Array(buffer, size));
		SendData(buffer, size);
	}

	//------------------------------------------------------------------------
	void CMemPro::StoreData(const void* p_data, int size)
	{
		MEMPRO_ASSERT(size < m_DataStorePageSize - (int)sizeof(DataStorePageHeader));

		if(!mp_DataStoreTail || mp_DataStoreTail->m_Size + size > m_DataStorePageSize)
		{
			DataStorePageHeader* p_new_page = (DataStorePageHeader*)Allocator::Alloc(m_DataStorePageSize);
			p_new_page->m_Size = sizeof(DataStorePageHeader);
			p_new_page->mp_Next = NULL;

			if(mp_DataStoreTail)
				mp_DataStoreTail->mp_Next = p_new_page;
			else
				mp_DataStoreHead = p_new_page;

			mp_DataStoreTail = p_new_page;
		}

		memcpy((char*)mp_DataStoreTail + mp_DataStoreTail->m_Size, p_data, size);
		mp_DataStoreTail->m_Size += size;
	}

	//------------------------------------------------------------------------
	void CMemPro::BlockUntilSendThreadEmpty()
	{
		// wait for the send thread to have sent all of the stored data
		while(m_Connected && m_RingBuffer.GetAllocatedRange(100).m_Size)
			Platform::Sleep(100);
	}

	//------------------------------------------------------------------------
	void CMemPro::SendStoredData()
	{
		if(!m_Connected)
			return;

		DataStorePageHeader* p_page = mp_DataStoreHead;

		if(p_page)
		{
			while(p_page)
			{
				DataStorePageHeader* p_next = p_page->mp_Next;

				SendData((char*)p_page + sizeof(DataStorePageHeader), p_page->m_Size - sizeof(DataStorePageHeader));
				Allocator::Free(p_page, m_DataStorePageSize);

				p_page = p_next;
			}

			SendPacketHeader(EDataStoreEndPacket);
			SendEndMarker();
		}

#ifndef MEMPRO_WRITE_DUMP
		BlockUntilSendThreadEmpty();
#endif

		mp_DataStoreHead = mp_DataStoreTail = NULL;
	}

	//------------------------------------------------------------------------
	void CMemPro::ClearStoreData()
	{
		DataStorePageHeader* p_page = mp_DataStoreHead;
		while(p_page)
		{
			DataStorePageHeader* p_next = p_page->mp_Next;
			Allocator::Free(p_page, m_DataStorePageSize);
			p_page = p_next;
		}

		mp_DataStoreHead = mp_DataStoreTail = NULL;

		m_CallstackSet.Clear();
	}

	//------------------------------------------------------------------------
	void CMemPro::Send(bool value)
	{
		unsigned int uint_value = value ? 1 : 0;
		Send(uint_value);
	}

	//------------------------------------------------------------------------
	bool CMemPro::SendThreadStillAlive() const
	{
		return m_SendThread.IsAlive();
	}

	//------------------------------------------------------------------------
	void CMemPro::FlushRingBufferForShutdown()
	{
		if(m_FlushedRingBufferForShutdown)
			return;
		m_FlushedRingBufferForShutdown = true;

		RingBuffer::Range range = m_RingBuffer.GetAllocatedRange(100);
		while(range.m_Size)
		{
			SocketSendData(range.mp_Buffer, range.m_Size);
			range = m_RingBuffer.GetAllocatedRange(100);
		}
	}

	//------------------------------------------------------------------------
	void CMemPro::SendData(const void* p_data, int size)
	{
		MEMPRO_ASSERT((size & 3) == 0);

		if(!m_Connected)
		{
			StoreData(p_data, size);
			return;
		}

		if(!SendThreadStillAlive())
		{
			FlushRingBufferForShutdown();
			SocketSendData(p_data, size);
		}
		else
		{
			int bytes_to_copy = size;
			char* p_src = (char*)p_data;
			while(bytes_to_copy)
			{
				RingBuffer::Range range;
				do {
					range = m_RingBuffer.GetFreeRange(100);
					if(!m_Connected)
						return;
				} while(!range.m_Size);
				if(!m_Connected)
					return;

				int copy_size = Min(range.m_Size, bytes_to_copy);
				SmallFastMemCpy(range.mp_Buffer, p_src, copy_size);
				p_src += copy_size;
				bytes_to_copy -= copy_size;

				m_RingBuffer.Add(copy_size);
			}
		}
	}

	//------------------------------------------------------------------------
	// slightly more optimal version for sending a single uint. Because all ringbuffer
	// operations are 4 byte aligned we can be guaranteed that the uint won't be split
	// between the end and start of the buffer, we will always get it in one piece.
	void CMemPro::SendData(unsigned int value)
	{
		if(!m_Connected)
		{
			StoreData(&value, sizeof(value));
			return;
		}

		if(!SendThreadStillAlive())
		{
			FlushRingBufferForShutdown();
			SocketSendData(&value, sizeof(value));
#ifdef MEMPRO_WRITE_DUMP
			g_DumpFile.Flush();
#endif
		}
		else
		{
			RingBuffer::Range range;
			do {
				range = m_RingBuffer.GetFreeRange(100);
				if(!m_Connected)
					return;
			} while(!range.m_Size);
			if(!m_Connected)
				return;

			MEMPRO_ASSERT(range.m_Size >= (int)sizeof(unsigned int));
			MEMPRO_ASSERT((((size_t)range.mp_Buffer) & 3) == 0);
			*(unsigned int*)range.mp_Buffer = value;

			m_RingBuffer.Add(sizeof(value));
		}
	}

	//------------------------------------------------------------------------
	void CMemPro::SendPacketHeader(PacketType value)
	{
		SendStartMarker();

		PacketHeader header;
		header.m_PacketType = value;
		header.m_Time = GetTime();

		Send(header);
	}

	//------------------------------------------------------------------------
	void CMemPro::SendStartMarker()
	{
#ifdef PACKET_START_END_MARKERS
		unsigned int start_marker = 0xabcdef01;
		ENDIAN_TEST(Platform::SwapEndian(start_marker));
		Send(start_marker);
#endif
	}

	//------------------------------------------------------------------------
	void CMemPro::SendEndMarker()
	{
#ifdef PACKET_START_END_MARKERS
		unsigned int end_marker = 0xaabbccdd;
		ENDIAN_TEST(Platform::SwapEndian(end_marker));
		Send(end_marker);
#endif
	}

	//------------------------------------------------------------------------
	void CMemPro::SendPageState(bool send_memory)
	{
		CriticalSectionScope lock(m_CriticalSection);

		SendPacketHeader(EPageStateStartPacket);
		SendEndMarker();

		Platform::SendPageState(send_memory, StaticSendPageState, this);

		SendPacketHeader(EPageStateEndPacket);

		IgnoreMemRangePacket range_packet;
		range_packet.m_Addr = ToUInt64(m_RingBufferMem);
		range_packet.m_Size = sizeof(m_RingBufferMem);
		Send(range_packet);

		SendEndMarker();
	}

	//------------------------------------------------------------------------
	void CMemPro::SendVMemStats()
	{
#ifdef VMEM_ENABLE_STATS
		struct MemProPacketHeader
		{
			PacketType m_PacketType;
			int m_Padding;
			int64 m_Time;
		};
		MemProPacketHeader header;
		header.m_PacketType = EVMemStats;
		header.m_Padding = 0;
		header.m_Time = GetTime();

		Send(header);

		Send(header.m_Time);

		VMem::SendStatsToMemPro(StaticSendVMemStatsData, this);
#endif
	}

	//------------------------------------------------------------------------
	void CMemPro::SendVirtualMemStats()
	{
		size_t reserved = 0;
		size_t committed = 0;

		Platform::GetVirtualMemStats(reserved, committed);

		SendPacketHeader(EVirtualMemStats);

		VirtualMemStatsPacket packet;
		packet.m_Reserved = reserved;
		packet.m_Committed = committed;
		ENDIAN_TEST(packet.SwapEndian());
		Send(packet);

		SendEndMarker();
	}

	//------------------------------------------------------------------------
	void** CMemPro::AllocateStackTraceData()
	{
		CriticalSectionScope lock(m_CriticalSection);
		return (void**)m_BlockAllocator.Alloc(Platform::GetStackTraceSize()*sizeof(void*));
	}

	//------------------------------------------------------------------------
	CallstackCapture CMemPro::CaptureCallstack()
	{
		CallstackCapture callstack;

		callstack.mp_Stack = g_CallstackDataTLS;
		if(!callstack.mp_Stack)
		{
			callstack.mp_Stack = AllocateStackTraceData();
			g_CallstackDataTLS = callstack.mp_Stack;
		}

		callstack.m_Hash = 0;
		callstack.m_Size = 0;
		GetStackTrace(callstack.mp_Stack, callstack.m_Size, callstack.m_Hash);

		const int ignore_count = 2;

		callstack.m_Size -= ignore_count;
		if(callstack.m_Size <= 0)
		{
			callstack.mp_Stack[0] = (void*)-1;
			callstack.m_Size = 1;
		}

		return callstack;
	}

	//------------------------------------------------------------------------
	int CMemPro::SendCallstack(const CallstackCapture& callstack_capture)
	{
		void** p_stack = callstack_capture.mp_Stack;
		int stack_size = callstack_capture.m_Size;
		int hash = callstack_capture.m_Hash;

#ifdef MEMPRO64
		uint64* stack64 = (uint64*)p_stack;
#else
		static uint64 stack64_static[STACK_TRACE_SIZE];
		for(int i=0; i<stack_size; ++i)
			stack64_static[i] = ToUInt64(p_stack[i]);
		uint64* stack64 = stack64_static;
#endif

		Callstack* p_callstack = m_CallstackSet.Get(stack64, stack_size, hash);

		if(!p_callstack)
		{
			p_callstack = m_CallstackSet.Add(stack64, stack_size, hash);

			SendPacketHeader(ECallstackPacket);

			int callstack_id = p_callstack->m_ID;
#ifdef TEST_ENDIAN
			Platform::SwapEndian(callstack_id);
#endif
			Send(callstack_id);

			int send_stack_size = stack_size;
#ifdef TEST_ENDIAN
			for(int i=0; i<stack_size; ++i) Platform::SwapEndian(stack64[i]);
			Platform::SwapEndian(send_stack_size);
#endif
			Send(send_stack_size);
			SendData(stack64, stack_size*sizeof(uint64));

			SendEndMarker();
		}

		return p_callstack->m_ID;
	}

	//------------------------------------------------------------------------
	void CMemPro::TakeSnapshot(bool send_memory)
	{
		SendPageState(send_memory);

		{
			CriticalSectionScope lock(m_CriticalSection);
			
			SendPacketHeader(ETakeSnapshot);

			TakeSnapshotPacket packet;
			packet.m_IsMemorySnapshot = send_memory;
			ENDIAN_TEST(packet.SwapEndian());
			Send(packet);
		}
	}

	//------------------------------------------------------------------------
	void CMemPro::FlushDumpFile()
	{
#ifdef MEMPRO_WRITE_DUMP
		CriticalSectionScope lock(m_CriticalSection);

		g_DumpFile.Flush();
#endif
	}

	//------------------------------------------------------------------------
	int CMemPro::SendThreadMainStatic(void* p_param)
	{
		return gp_MemPro->SendThreadMain(p_param);
	}

	//------------------------------------------------------------------------
	bool CMemPro::SocketSendData(const void* p_data, int size)
	{
#ifdef MEMPRO_WRITE_DUMP
		bool result = g_DumpFile.Write(p_data, size);
		MEMPRO_ASSERT(result);
		return true;
#else
		return m_ClientSocket.Send((void*)p_data, size);
#endif
	}

	//------------------------------------------------------------------------
	int CMemPro::SendThreadMain(void* p_param)
	{
		while(m_Connected)
		{
			RingBuffer::Range range;
			do {
				range = m_RingBuffer.GetAllocatedRange(100);	// timeout: check for disconnect every 100 ms
				if(!m_Connected)
				{
					m_SendThreadFinishedEvent.Set();
					return 0;
				}
			} while(!range.m_Size);

			if(!SocketSendData(range.mp_Buffer, range.m_Size))
			{
				m_SendThreadFinishedEvent.Set();
				Disconnect(true);
				return 0;
			}

			m_RingBuffer.Remove(range.m_Size);
		}

		m_SendThreadFinishedEvent.Set();
		return 0;
	}

	//------------------------------------------------------------------------
#ifndef MEMPRO_WRITE_DUMP
	int CMemPro::ReceiveThreadMainStatic(void* p_param)
	{
		return gp_MemPro->ReceiveThreadMain(p_param);
	}
#endif

	//------------------------------------------------------------------------
#ifndef MEMPRO_WRITE_DUMP
	int CMemPro::ReceiveThreadMain(void* p_param)
	{
		while(m_Connected)
		{
			unsigned int flag = 0;

			if(m_ClientSocket.Receive(&flag, sizeof(flag)) != sizeof(flag))
			{
				m_ReceiveThreadFinishedEvent.Set();
				Disconnect(true);
				return 0;
			}

			switch(flag)
			{
				case SendPageData: SendPageState(false/*send_memory*/); break;
				case SendPageDataWithMemory: SendPageState(true/*send memory*/); break;
				case EShutdownComplete: m_MemProReadyToShutdownEvent.Set(); break;
			}
		}

		m_ReceiveThreadFinishedEvent.Set();
		return 0;
	}
#endif

	//------------------------------------------------------------------------
	struct MemProGUID
	{
		unsigned long  Data1;
		unsigned short Data2;
		unsigned short Data3;
		unsigned char  Data4[8];
	};

	//------------------------------------------------------------------------
	void CMemPro::SendExtraModuleInfo(int64 ModuleBase)
	{
		int age = 0;
		MemProGUID signature;
		char pdb_filename[512] = { 0 };

		bool get_extra_modules_result = Platform::GetExtraModuleInfo(
			ModuleBase,
			age,
			&signature,
			sizeof(signature),
			pdb_filename,
			sizeof(pdb_filename));

		if (get_extra_modules_result)
		{
			Send(true);				// sending info
			Send(age);
			Send(signature);
			SendString(pdb_filename);
		}
		else
		{
			// failed to find info
			Send(false);				// not sending info
		}
	}

	//------------------------------------------------------------------------
	void CMemPro::SendString(const char* p_str)
	{
		const int max_path_len = 1024;
		int len = (int)strlen(p_str) + 1;
		MEMPRO_ASSERT(len <= max_path_len);

		// round up to 4 bytes
		static char temp[max_path_len+1] MEMPRO_ALIGN_SUFFIX(16);
		memset(temp, 0, sizeof(temp));
		memcpy(temp, p_str, len);

		int rounded_len = ((int)len + 3) & ~3;
		Send(rounded_len);

		SendData(temp, rounded_len);
	}

	//------------------------------------------------------------------------
	void CMemPro::StaticEnumerateLoadedModulesCallback(int64 module_base, const char* p_module_name, void* p_context)
	{
		CMemPro* p_this = (CMemPro*)p_context;
		p_this->EnumerateLoadedModulesCallback(module_base, p_module_name);
	}

	//------------------------------------------------------------------------
	void CMemPro::EnumerateLoadedModulesCallback(int64 module_base, const char* p_module_name)
	{
		Send(module_base);

		// if we send the special "use function lookup address" marker we need to
		// send the function address next
		if (module_base == 0xabcdefabcdef1LL)
		{
			int64 function_lookup_address = (int64)BaseAddressLookupFunction;
			Send(function_lookup_address);
		}

		SendString(p_module_name);

		SendExtraModuleInfo(module_base);

		++m_ModulesSent;
	}

	//------------------------------------------------------------------------
	void CMemPro::SendModuleInfo()
	{
		Send(true);

		// indicate we are going to be sending module signatures - for backwards compatibility
		uint64 extra_module_info = 0xabcdef;
		Send(extra_module_info);

		m_ModulesSent = 0;

		// if you are having problems compiling this on your platform undefine ENUMERATE_ALL_MODULES and it send info for just the main module
		Platform::MemProEnumerateLoadedModules(StaticEnumerateLoadedModulesCallback, this);

		uint64 terminator = 0xffffffffffffffffULL;
		Send(terminator); //-V569
	}

	//------------------------------------------------------------------------
	bool CMemPro::WaitForConnection()
	{
		m_CriticalSection.Enter();

#ifdef MEMPRO_WRITE_DUMP
		char dump_filename[256];
		Platform::GetDumpFilename(dump_filename, 256);

		char log_message[256];
		Platform::SPrintF(log_message, (int)sizeof(log_message), "MemPro writing to dump file %s\n", dump_filename);
		Platform::DebugWrite(log_message);

		bool open_result = g_DumpFile.OpenForWrite(dump_filename);
		MEMPRO_ASSERT(open_result);

		m_Connected = true;

		m_SendThreadFinishedEvent.Reset();

		// start the sending thread
		int thread_id = m_SendThread.CreateThread(SendThreadMainStatic);
		SetThreadName(thread_id, "MemPro write thread");
#else
		char log_message[256];
		Platform::SPrintF(log_message, sizeof(log_message), "MemPro listening on port %s\n", g_DefaultPort);
		Platform::DebugWrite(log_message);

		// start listening for connections
		if(m_ListenSocket.IsValid() && !m_ListenSocket.StartListening())
		{
			m_WaitForConnectThreadFinishedEvent.Set();		// do this before Shutdown
			Shutdown();
			m_CriticalSection.Leave();
			return false;
		}

		m_StartedListening = true;
		m_StartedListeningEvent.Set();

		// Accept a client socket
		bool accepted = false;
		if(m_ListenSocket.IsValid())
		{
			m_CriticalSection.Leave();
			accepted = m_ListenSocket.Accept(m_ClientSocket);

			if(!accepted)
			{
				bool shutting_down = m_ShuttingDown;
				m_WaitForConnectThreadFinishedEvent.Set();		// do this before Shutdown
				if(!shutting_down)			// check shutting down here in case CMemPro has been destructed
				{
					m_CriticalSection.Enter();
					Shutdown();
					m_CriticalSection.Leave();
				}
				return false;
			}
		}

		m_CriticalSection.Enter();

		m_Connected = true;

		m_SendThreadFinishedEvent.Reset();
		m_ReceiveThreadFinishedEvent.Reset();

		// start the sending thread
		int send_thread_id = m_SendThread.CreateThread(SendThreadMainStatic);
		SetThreadName(send_thread_id, "MemPro send thread");

		// start the receiving thread
		int receive_thread_id = m_ReceiveThread.CreateThread(ReceiveThreadMainStatic);
		SetThreadName(receive_thread_id, "MemPro receive thread");
#endif
		// send the connect key
		unsigned int endian_key = (unsigned int)EndianKey;
		ENDIAN_TEST(Platform::SwapEndian(endian_key));
		Send(endian_key);

		// send the connect packet
		ConnectPacket connect_packet;
		connect_packet.m_Padding = 0xabcdabcd;
		connect_packet.m_Version = MemPro::Version;
		connect_packet.m_TickFrequency = GetTickFrequency();
		connect_packet.m_ConnectTime = GetTime();

		connect_packet.m_PtrSize = sizeof(void*);

		connect_packet.m_Platform = Platform::GetPlatform();

		ENDIAN_TEST(connect_packet.SwapEndian());
		Send(connect_packet);

		SendModuleInfo();

		Platform::MemProMemoryBarrier();

		SendStoredData();

		m_ReadyToSend = true;

		m_WaitForConnectThreadFinishedEvent.Set();
		m_CriticalSection.Leave();

		// start the pulse thread
		m_PulseThreadFinished.Reset();
		int pulse_thread_id = m_PulseThread.CreateThread(PulseThreadMainStatic);
		SetThreadName(pulse_thread_id, "MemPro pulse thread");

		return true;
	}

	//------------------------------------------------------------------------
	int CMemPro::WaitForConnectionThreadMainStatic(void* p_param)
	{
		return gp_MemPro->WaitForConnectionThreadMain(p_param);
	}

	//------------------------------------------------------------------------
	int CMemPro::PulseThreadMainStatic(void* p_param)
	{
		gp_MemPro->PulseThreadMain();
		return 0;
	}

	//------------------------------------------------------------------------
	void CMemPro::PulseThreadMain()
	{
		while(m_Connected)
		{
			{
				CriticalSectionScope lock(m_CriticalSection);
				if(!m_Connected)
					break;

				SendPacketHeader(EPulsePacket);
				SendEndMarker();
			}

			Platform::Sleep(1000);
		}

		m_PulseThreadFinished.Set();
	}

	//------------------------------------------------------------------------
	int CMemPro::WaitForConnectionThreadMain(void* p_param)
	{
#ifdef MEMPRO_WRITE_DUMP
		Platform::Sleep(MEMPRO_INIT_DELAY);
#else
		if(!m_ListenSocket.IsValid())
		{
			Platform::Sleep(MEMPRO_INIT_DELAY);
			
			bool bind_result = m_ListenSocket.Bind(g_DefaultPort);
			
			if(!bind_result)
				Platform::DebugWrite("MemPro ERROR: Failed to bind port. This usually means that another process is already running with MemPro enabled.\n");
			MEMPRO_ASSERT(bind_result);
			if(!bind_result)
				return 0;
		}
#endif
		WaitForConnection();

		return 0;
	}

	//------------------------------------------------------------------------
	bool CMemPro::Initialise()
	{
		m_WaitForConnectionThread.CreateThread(WaitForConnectionThreadMainStatic);

		return true;
	}

	//------------------------------------------------------------------------
	void CMemPro::Shutdown()
	{
		m_ShuttingDown = true;

		// wait for MemPro to have handled all data
		if(m_SendThread.IsAlive())
		{
			SendPacketHeader(ERequestShutdown);
			SendEndMarker();
			m_MemProReadyToShutdownEvent.Wait(10 * 1000);

			// do this so that we don't start listening after the listen socket has been shutdown and deadlock
			m_CriticalSection.Leave();
			m_StartedListeningEvent.Wait();
			m_CriticalSection.Enter();

			if(m_WaitForConnect)
			{
				BlockUntilReadyToSend();
				BlockUntilSendThreadEmpty();
			}
		}

		Disconnect(false/*listen_for_new_connection*/);

		m_CriticalSection.Leave();
		m_PulseThreadFinished.Wait();
		m_CriticalSection.Enter();

#ifndef MEMPRO_WRITE_DUMP
		m_ListenSocket.Disconnect();
	
		if(m_WaitForConnectionThread.IsAlive())
			m_WaitForConnectThreadFinishedEvent.Wait(1000);

		Platform::UninitialiseSockets();
#endif
	}

	//------------------------------------------------------------------------
	void CMemPro::Disconnect(bool listen_for_new_connection)
	{
		CriticalSectionScope lock(m_DisconnectCriticalSection);

		if(m_Connected)
		{
			m_ReadyToSend = false;
			m_Connected = false;

			// wait for the send thread to shutdown
			m_SendThreadFinishedEvent.Wait();
			m_SendThreadFinishedEvent.Reset();

#ifdef MEMPRO_WRITE_DUMP
			g_DumpFile.Close();
#else
			// close the client socket
			m_ClientSocket.Disconnect();

			// wait for the receive thread to shutdown
			m_ReceiveThreadFinishedEvent.Wait();
			m_ReceiveThreadFinishedEvent.Reset();
#endif
			// clear stuff
			m_CallstackSet.Clear();

			m_RingBuffer.Clear();

#ifndef MEMPRO_WRITE_DUMP
			if(listen_for_new_connection)
			{
				CriticalSectionScope lock2(m_CriticalSection);

				// start listening for another connection
				m_ListenSocket.Disconnect();
				m_StartedListeningEvent.Reset();
				m_StartedListening = false;
				m_InitialConnectionTimedOut = false;
				m_WaitForConnectionThread.CreateThread(WaitForConnectionThreadMainStatic);
			}
#endif
		}
	}

	//------------------------------------------------------------------------
	void CMemPro::BlockUntilReadyToSend()
	{
#ifndef MEMPRO_WRITE_DUMP
		if(m_ListenSocket.IsValid())
		{
			Platform::DebugWrite("Waiting for connection to MemPro...\n");

			int64 start_time = GetTime();
			while(!m_ReadyToSend && m_ListenSocket.IsValid() &&
				(m_WaitForConnect || ((GetTime() - start_time) / (double)GetTickFrequency()) * 1000 < MEMPRO_CONNECT_TIMEOUT))
			{
				m_CriticalSection.Leave();
				Platform::Sleep(100);
				m_CriticalSection.Enter();
			}

			if(m_ReadyToSend)
			{
				Platform::DebugWrite("Connected to MemPro!\n");
			}
			else
			{
				m_InitialConnectionTimedOut = true;
				ClearStoreData();
				Platform::DebugWrite("Failed to connect to MemPro\n");
			}
		}
#endif
	}

	//------------------------------------------------------------------------
	// return true to continue processing event (either connected or before started listening)
	bool CMemPro::WaitForConnectionIfListening()
	{
#ifdef MEMPRO_WRITE_DUMP
		return true;
#else
		if(!m_ReadyToSend && !m_InitialConnectionTimedOut)
		{
			CriticalSectionScope lock(m_CriticalSection);

			// store data until we have started listening
			if(!m_StartedListening)
				return true;

			BlockUntilReadyToSend();
		}

		return m_ReadyToSend;
#endif
	}

	//------------------------------------------------------------------------
	void CMemPro::TrackAlloc(void* p, size_t size, bool wait_for_connect)
	{
		if(m_Paused)
			return;

		m_WaitForConnect = wait_for_connect;

		if(!WaitForConnectionIfListening())
			return;

		CallstackCapture callstack_capture = CaptureCallstack();

		CriticalSectionScope lock(m_CriticalSection);

#ifndef MEMPRO_WRITE_DUMP
		if(m_ListenSocket.IsValid())
#endif
		{
			int now = (int)((Platform::GetHiResTimer() * 1000) / Platform::GetHiResTimerFrequency());
			if(now - m_LastPageStateSend > m_PageStateInterval)
			{
				SendVirtualMemStats();
				m_LastPageStateSend = now;
			}

			if(now - m_LastVMemStatsSend > m_VMemStatsSendInterval)
			{
				SendVMemStats();
				m_LastVMemStatsSend = now;
			}
		}

		if(m_InEvent)
			return;
		m_InEvent = true;

		int callstack_id = SendCallstack(callstack_capture);

		SendPacketHeader(EAllocPacket);

		AllocPacket packet;
		packet.m_Addr = ObfuscateAddress(ToUInt64(p));
		packet.m_Size = size;
		packet.m_CallstackID = callstack_id;
		packet.m_Padding = 0xef12ef12;
		ENDIAN_TEST(packet.SwapEndian());
		Send(packet);

		SendEndMarker();

		m_InEvent = false;
	}

	//------------------------------------------------------------------------
	void CMemPro::TrackFree(void* p, bool wait_for_connect)
	{
		if(m_Paused)
			return;

		m_WaitForConnect = wait_for_connect;

		if(!WaitForConnectionIfListening())
			return;

		CriticalSectionScope lock(m_CriticalSection);

		if(m_InEvent)
			return;
		m_InEvent = true;

		SendPacketHeader(EFreePacket);

		FreePacket packet;
		packet.m_Addr = ObfuscateAddress(ToUInt64(p));
		ENDIAN_TEST(packet.SwapEndian());
		Send(packet);

		SendEndMarker();

		m_InEvent = false;
	}

	//------------------------------------------------------------------------
	bool CMemPro::IsPaused()
	{
		return m_Paused;
	}

	//------------------------------------------------------------------------
	void CMemPro::SetPaused(bool paused)
	{
		m_Paused = paused;
	}

	//------------------------------------------------------------------------
	void CMemPro::StaticSendPageState(void* p, size_t size, PageState page_state, PageType page_type, unsigned int page_protection, bool send_page_mem, int page_size, void* p_context)
	{
		CMemPro* p_this = (CMemPro*)p_context;
		p_this->SendPageState(p, size, page_state, page_type, page_protection, send_page_mem, page_size);
	}

	//------------------------------------------------------------------------
	void CMemPro::SendPageState(void* p, size_t size, PageState page_state, PageType page_type, unsigned int page_protection, bool send_page_mem, int page_size)
	{
		if(!WaitForConnectionIfListening())
			return;

		SendPacketHeader(EPageStatePacket);

		PageStatePacket packet;
		packet.m_Addr = ToUInt64(p);
		packet.m_Size = size;
		packet.m_State = page_state;
		packet.m_Type = page_type;
		packet.m_Protection = page_protection;
		packet.m_SendingMemory = send_page_mem;
		ENDIAN_TEST(packet.SwapEndian());
		Send(packet);

		if(send_page_mem)
		{
			MEMPRO_ASSERT(!(size % page_size));
			char* p_page = (char*)p;
			char* p_end_page = p_page + size;
			while(p_page != p_end_page)
			{
				SendData(p_page, page_size);
				p_page += page_size;
			}
		}

		SendEndMarker();
	}

	//------------------------------------------------------------------------
	void CMemPro::WaitForConnectionOnInitialise()
	{
		m_WaitForConnect = true;

		m_StartedListeningEvent.Wait();

		CriticalSectionScope lock(m_CriticalSection);
		BlockUntilReadyToSend();
	}
}

//------------------------------------------------------------------------
void MemPro::InitialiseInternal()
{
	static CriticalSection lock;
	CriticalSectionScope scope_lock(lock);
	
	if(!gp_MemPro && !g_ShuttingDown)
	{
		CMemPro* p_mempro = (CMemPro*)g_MemProMem;
		UE_CLANG_SUPPRESS //warning: Storage type is aligned to 1 bytes but allocated type is aligned to 8 bytes [cplusplus.PlacementNew]
		new (p_mempro)CMemPro();
		gp_MemPro = p_mempro;
		p_mempro->Initialise();
	}
}

//------------------------------------------------------------------------
void MemPro::IncRef()
{
	++g_MemProRefs;
}

//------------------------------------------------------------------------
void MemPro::DecRef()
{
	if(--g_MemProRefs == 0)
		Shutdown();
}

//------------------------------------------------------------------------
// called by the APP (not internally)
void MemPro::Initialise(bool wait_for_connect)
{
	InitialiseInternal();

	if(wait_for_connect)
		gp_MemPro->WaitForConnectionOnInitialise();
}

//------------------------------------------------------------------------
void MemPro::Disconnect()
{
	if(gp_MemPro)
	{
		gp_MemPro->Lock();
		gp_MemPro->Disconnect(true);
		gp_MemPro->Release();
	}
}

//------------------------------------------------------------------------
void MemPro::Shutdown()
{
	if(!g_ShuttingDown)
	{
		g_ShuttingDown = true;
		if(gp_MemPro)
		{
			gp_MemPro->Lock();
			gp_MemPro->Shutdown();
			gp_MemPro->Release();
			gp_MemPro->~CMemPro();
			memset((void*)gp_MemPro, 0, sizeof(CMemPro));
			gp_MemPro = NULL;
		}
	}
}

//------------------------------------------------------------------------
void MemPro::TrackAlloc(void* p, size_t size, bool wait_for_connect)
{
	CMemPro* p_mempro = GetMemPro();
	if(p_mempro)
		p_mempro->TrackAlloc(p, size, wait_for_connect);
}

//------------------------------------------------------------------------
void MemPro::TrackFree(void* p, bool wait_for_connect)
{
	CMemPro* p_mempro = GetMemPro();
	if(p_mempro)
		p_mempro->TrackFree(p, wait_for_connect);
}

//------------------------------------------------------------------------
void MemPro::SetPaused(bool paused)
{
	CMemPro* p_mempro = GetMemPro();
	if(p_mempro)
		p_mempro->SetPaused(paused);
}

//------------------------------------------------------------------------
bool MemPro::IsPaused()
{
	CMemPro* p_mempro = GetMemPro();
	return p_mempro ? p_mempro->IsPaused() : false;
}

//------------------------------------------------------------------------
void MemPro::TakeSnapshot(bool send_memory)
{
	if(gp_MemPro) gp_MemPro->TakeSnapshot(send_memory);
}

//------------------------------------------------------------------------
void MemPro::FlushDumpFile()
{
	if (gp_MemPro) gp_MemPro->FlushDumpFile();
}

//------------------------------------------------------------------------
#ifdef MEMPRO_ENABLE_WARNING_PRAGMAS
	MEMPRO_POP_WARNING_DISABLE
#endif

//------------------------------------------------------------------------
#endif		// #if MEMPRO_ENABLED

//------------------------------------------------------------------------
//
// MemProPlatform.cpp
//
//------------------------------------------------------------------------
#if MEMPRO_ENABLED

//------------------------------------------------------------------------
#include <new>

//------------------------------------------------------------------------
namespace MemPro
{
	//------------------------------------------------------------------------
	inline unsigned int GetHash(void** p_stack, int stack_size)
	{
#ifdef MEMPRO64
		const unsigned int prime = 0x01000193;
		unsigned int hash = prime;
		void** p = p_stack;
		for(int i=0; i<stack_size; ++i)
		{
			uint64 key = (uint64)(*p++);
			key = (~key) + (key << 18);
			key = key ^ (key >> 31);
			key = key * 21;
			key = key ^ (key >> 11);
			key = key + (key << 6);
			key = key ^ (key >> 22);
			hash = hash ^ (unsigned int)key;
		}

		return hash;
#else
		const unsigned int prime = 0x01000193;
		unsigned int hash = prime;
		for(int i=0; i<stack_size; ++i)
			hash = (hash * prime) ^ (unsigned int)p_stack[i];

		return hash;
#endif
	}

	//------------------------------------------------------------------------
	unsigned int GetHashAndStackSize(void** p_stack, int& stack_size)
	{
#ifdef MEMPRO64
		const unsigned int prime = 0x01000193;
		unsigned int hash = prime;
		stack_size = 0;
		void** p = p_stack;
		while (*p)
		{
			uint64 key = (uint64)(*p++);
			key = (~key) + (key << 18);
			key = key ^ (key >> 31);
			key = key * 21;
			key = key ^ (key >> 11);
			key = key + (key << 6);
			key = key ^ (key >> 22);
			hash = hash ^ (unsigned int)key;
			++stack_size;
		}

		return hash;
#else
		const unsigned int prime = 0x01000193;
		unsigned int hash = prime;
		stack_size = 0;
		while (p_stack[stack_size])
		{
			hash = (hash * prime) ^ (unsigned int)p_stack[stack_size];
			++stack_size;
		}

		return hash;
#endif
	}
}

//------------------------------------------------------------------------
#if defined(MEMPRO_WIN_BASED_PLATFORM)

//------------------------------------------------------------------------
#include <tchar.h>
#include <stdio.h>

#if !((!defined(MIDL_PASS) && defined(_M_IX86) && !defined(_M_CEE_PURE)) || defined(MemoryBarrier))
	#include <atomic>
#endif

//------------------------------------------------------------------------
// if both of these options are commented out it will use CaptureStackBackTrace (or backtrace on linux)
//#define USE_STACKWALK64				// much slower but possibly more reliable. USE_STACKWALK64 only implemented for x86 builds.
//#define USE_RTLVIRTUALUNWIND			// reported to be faster than StackWalk64 - only available on x64 builds

//------------------------------------------------------------------------
#if !defined(MEMPRO_WRITE_DUMP)
	#pragma comment(lib, "Ws2_32.lib")
#endif

//------------------------------------------------------------------------

//------------------------------------------------------------------------
// USE_INTRINSIC can also be enabled on 32bit platform, but I left it disabled because it doesn't work on XP
#ifdef MEMPRO64
	#define USE_INTRINSIC
#endif

//------------------------------------------------------------------------
#ifdef USE_INTRINSIC
	#include <intrin.h>
	#pragma intrinsic(_InterlockedCompareExchange64)
	#pragma intrinsic(_InterlockedExchangeAdd64)
#endif

//------------------------------------------------------------------------
#include <tchar.h>

//------------------------------------------------------------------------
#pragma warning(push)
#pragma warning(disable : 4100)
#pragma warning(disable : 4189)

//------------------------------------------------------------------------
#ifdef USE_RTLCAPTURESTACKBACKTRACE

	#ifndef MEMPRO64
		#error USE_RTLVIRTUALUNWIND only available on x64 builds. Please use a different stack walk function.
	#endif

	//------------------------------------------------------------------------
	namespace MemPro
	{
		//------------------------------------------------------------------------
		void RTLCaptureStackBackTrace(void** stack, int max_stack_size, unsigned int& hash, int& stack_size)
		{
			memset(stack, 0, max_stack_size* sizeof(void*));
			stack_size = ::RtlCaptureStackBackTrace(1,max_stack_size-1, stack, (PDWORD)&hash);
			stack[stack_size] = 0;
		}
	}

#endif		// #ifdef USE_RTLCAPTURESTACKBACKTRACE

//------------------------------------------------------------------------
namespace MemPro
{
	//------------------------------------------------------------------------
	// http://www.debuginfo.com/articles/debuginfomatch.html
	struct CV_HEADER
	{
		int Signature;
		int Offset;
	};

	struct CV_INFO_PDB20
	{
		CV_HEADER CvHeader;
		int Signature;
		int Age;
		char PdbFileName[MAX_PATH];
	};

	struct CV_INFO_PDB70
	{
		int  CvSignature;
		GUID Signature;
		int Age;
		char PdbFileName[MAX_PATH];
	};

	//------------------------------------------------------------------------
	namespace GenericPlatform
	{
		//------------------------------------------------------------------------
		CRITICAL_SECTION& GetOSLock(void* p_os_lock_mem)
		{
			return *(CRITICAL_SECTION*)p_os_lock_mem;
		}

		//------------------------------------------------------------------------
		void CreateLock(void* p_os_lock_mem, int os_lock_mem_size)
		{
			MEMPRO_ASSERT(os_lock_mem_size >= sizeof(CRITICAL_SECTION));
			InitializeCriticalSection(&GetOSLock(p_os_lock_mem));
		}

		//------------------------------------------------------------------------
		void DestroyLock(void* p_os_lock_mem)
		{
			DeleteCriticalSection(&GetOSLock(p_os_lock_mem));
		}

		//------------------------------------------------------------------------
		void TakeLock(void* p_os_lock_mem)
		{
			EnterCriticalSection(&GetOSLock(p_os_lock_mem));
		}

		//------------------------------------------------------------------------
		void ReleaseLock(void* p_os_lock_mem)
		{
			LeaveCriticalSection(&GetOSLock(p_os_lock_mem));
		}

		//------------------------------------------------------------------------
		#ifndef MEMPRO_WRITE_DUMP
		void HandleError()
		{
			if (WSAGetLastError() == WSAEADDRINUSE)
			{
				OutputDebugString(_T("MemPro: Network connection conflict. Please make sure that other MemPro enabled applications are shut down, or change the port in the the MemPro lib and MemPro settings.\n"));
				return;
			}

			int buffer_size = 1024;
			TCHAR* p_buffer = (TCHAR*)HeapAlloc(GetProcessHeap(), 0, buffer_size * sizeof(TCHAR));
			memset(p_buffer, 0, buffer_size * sizeof(TCHAR));

			va_list args;
			DWORD result = FormatMessage(
				FORMAT_MESSAGE_FROM_SYSTEM,
				NULL,
				WSAGetLastError(),
				0,
				p_buffer,
				4 * 1024,
				&args);

			OutputDebugString(p_buffer);

			HeapFree(GetProcessHeap(), 0, p_buffer);
		}
		#endif

		//------------------------------------------------------------------------
		#ifndef MEMPRO_WRITE_DUMP
		bool g_SocketsInitialised = false;
		#endif

		//------------------------------------------------------------------------
		#ifndef MEMPRO_WRITE_DUMP
		bool InitialiseSockets()
		{
			if (!g_SocketsInitialised)
			{
				g_SocketsInitialised = true;

				// Initialize Winsock
				WSADATA wsaData;
				if (WSAStartup(MAKEWORD(2, 2), &wsaData) != 0)
				{
					HandleError();
					return false;
				}
			}

			return true;
		}
		#endif

		//------------------------------------------------------------------------
		#ifndef MEMPRO_WRITE_DUMP
		void UninitialiseSockets()
		{
			if (g_SocketsInitialised)
			{
				g_SocketsInitialised = false;

				if (WSACleanup() == SOCKET_ERROR)
					HandleError();
			}
		}
		#endif

		//------------------------------------------------------------------------
		#ifndef MEMPRO_WRITE_DUMP
		void CreateSocket(void* p_os_socket_mem, int os_socket_mem_size)
		{
			MEMPRO_ASSERT(os_socket_mem_size >= sizeof(SOCKET));
			*(SOCKET*)p_os_socket_mem = INVALID_SOCKET;
		}
		#endif

		//------------------------------------------------------------------------
		#ifndef MEMPRO_WRITE_DUMP
		bool IsValidSocket(const void* p_os_socket_mem)
		{
			return *(const SOCKET*)p_os_socket_mem != INVALID_SOCKET;
		}
		#endif

		//------------------------------------------------------------------------
		#ifndef MEMPRO_WRITE_DUMP
		void Disconnect(void* p_os_socket_mem)
		{
			SOCKET& socket = *(SOCKET*)p_os_socket_mem;

			if (socket != INVALID_SOCKET)
			{
				if (shutdown(socket, SD_BOTH) == SOCKET_ERROR)
					HandleError();

				// loop until the socket is closed to ensure all data is sent
				unsigned int buffer = 0;
				size_t ret = 0;
				do { ret = recv(socket, (char*)&buffer, sizeof(buffer), 0); } while (ret != 0 && ret != (size_t)SOCKET_ERROR);

				if (closesocket(socket) == SOCKET_ERROR)
					HandleError();

				socket = INVALID_SOCKET;
			}
		}
		#endif

		//------------------------------------------------------------------------
		#ifndef MEMPRO_WRITE_DUMP
		bool StartListening(void* p_os_socket_mem)
		{
			SOCKET& socket = *(SOCKET*)p_os_socket_mem;

			MEMPRO_ASSERT(socket != INVALID_SOCKET);

			if (listen(socket, SOMAXCONN) == SOCKET_ERROR)
			{
				HandleError();
				return false;
			}
			return true;
		}
		#endif

		//------------------------------------------------------------------------
		#ifndef MEMPRO_WRITE_DUMP
		bool BindSocket(void* p_os_socket_mem, const char* p_port)
		{
			SOCKET& socket = *(SOCKET*)p_os_socket_mem;

			MEMPRO_ASSERT(socket == INVALID_SOCKET);

			if (!InitialiseSockets())
				return false;

			// setup the addrinfo struct
			addrinfo info;
			ZeroMemory(&info, sizeof(info));
			info.ai_family = AF_INET;
			info.ai_socktype = SOCK_STREAM;
			info.ai_protocol = IPPROTO_TCP;
			info.ai_flags = AI_PASSIVE;

			// Resolve the server address and port
			addrinfo* p_result_info;
			HRESULT result = getaddrinfo(NULL, p_port, &info, &p_result_info);
			if (result != 0)
			{
				HandleError();
				return false;
			}

			socket = ::socket(
				p_result_info->ai_family,
				p_result_info->ai_socktype,
				p_result_info->ai_protocol);

			if (socket == INVALID_SOCKET)
			{
				freeaddrinfo(p_result_info);
				HandleError();
				return false;
			}

			// Setup the TCP listening socket
			result = ::bind(socket, p_result_info->ai_addr, (int)p_result_info->ai_addrlen);
			freeaddrinfo(p_result_info);

			if (result == SOCKET_ERROR)
			{
				HandleError();
				Disconnect(p_os_socket_mem);
				return false;
			}

			return true;
		}
		#endif

		//------------------------------------------------------------------------
		#ifndef MEMPRO_WRITE_DUMP
		bool AcceptSocket(void* p_os_socket_mem, void* p_client_os_socket_mem)
		{
			SOCKET& socket = *(SOCKET*)p_os_socket_mem;
			SOCKET& client_socket = *(SOCKET*)p_client_os_socket_mem;

			MEMPRO_ASSERT(client_socket == INVALID_SOCKET);
			client_socket = accept(socket, NULL, NULL);

			return client_socket != INVALID_SOCKET;
		}
		#endif

		//------------------------------------------------------------------------
		#ifndef MEMPRO_WRITE_DUMP
		bool SocketSend(void* p_os_socket_mem, void* p_buffer, int size)
		{
			SOCKET& socket = *(SOCKET*)p_os_socket_mem;

			int bytes_to_send = size;
			while (bytes_to_send != 0)
			{
				int bytes_sent = (int)send(socket, (char*)p_buffer, bytes_to_send, 0);

				if (bytes_sent == SOCKET_ERROR)
				{
					HandleError();
					Disconnect(p_os_socket_mem);
					return false;
				}
				p_buffer = (char*)p_buffer + bytes_sent;
				bytes_to_send -= bytes_sent;
			}

			return true;
		}
		#endif

		//------------------------------------------------------------------------
		#ifndef MEMPRO_WRITE_DUMP
		int SocketReceive(void* p_os_socket_mem, void* p_buffer, int size)
		{
			SOCKET& socket = *(SOCKET*)p_os_socket_mem;

			int total_bytes_received = 0;
			while (size)
			{
				int bytes_received = (int)recv(socket, (char*)p_buffer, size, 0);

				total_bytes_received += bytes_received;

				if (bytes_received == 0)
				{
					Disconnect(p_os_socket_mem);
					return bytes_received;
				}
				else if (bytes_received == SOCKET_ERROR)
				{
					HandleError();
					Disconnect(p_os_socket_mem);
					return bytes_received;
				}

				size -= bytes_received;
				p_buffer = (char*)p_buffer + bytes_received;
			}

			return total_bytes_received;
		}
		#endif

		//------------------------------------------------------------------------
		void MemProCreateEvent(
			void* p_os_event_mem,
			int os_event_mem_size,
			bool initial_state,
			bool auto_reset)
		{
			MEMPRO_ASSERT(os_event_mem_size >= sizeof(HANDLE));
			HANDLE& handle = *(HANDLE*)p_os_event_mem;
			handle = CreateEvent(NULL, !auto_reset, initial_state, NULL);
		}

		//------------------------------------------------------------------------
		void DestroyEvent(void* p_os_event_mem)
		{
			HANDLE& handle = *(HANDLE*)p_os_event_mem;

			CloseHandle(handle);
		}

		//------------------------------------------------------------------------
		void SetEvent(void* p_os_event_mem)
		{
			HANDLE& handle = *(HANDLE*)p_os_event_mem;

			::SetEvent(handle);
		}

		//------------------------------------------------------------------------
		void ResetEvent(void* p_os_event_mem)
		{
			HANDLE& handle = *(HANDLE*)p_os_event_mem;

			::ResetEvent(handle);
		}

		//------------------------------------------------------------------------
		int WaitEvent(void* p_os_event_mem, int timeout)
		{
			HANDLE& handle = *(HANDLE*)p_os_event_mem;

			MEMPRO_STATIC_ASSERT(INFINITE == -1);
			return WaitForSingleObject(handle, timeout) == 0/*WAIT_OBJECT_0*/;
		}

		//------------------------------------------------------------------------
		struct PlatformThread
		{
			HANDLE m_Handle;
			bool m_Alive;
			ThreadMain mp_ThreadMain;
			void* mp_Param;
		};

		//------------------------------------------------------------------------
		void CreateThread(void* p_os_thread_mem, int os_thread_mem_size)
		{
			MEMPRO_ASSERT(os_thread_mem_size >= sizeof(PlatformThread));
			PlatformThread& platform_thread = *(PlatformThread*)p_os_thread_mem;
			platform_thread.m_Handle = 0;
			platform_thread.m_Alive = false;
			platform_thread.mp_ThreadMain = NULL;
			platform_thread.mp_Param = NULL;
		}

		//------------------------------------------------------------------------
		void DestroyThread(void* p_os_thread_mem)
		{
			PlatformThread& platform_thread = *(PlatformThread*)p_os_thread_mem;
			platform_thread.~PlatformThread();
		}

		//------------------------------------------------------------------------
		unsigned long WINAPI PlatformThreadMain(void* p_param)
		{
			PlatformThread& platform_thread = *(PlatformThread*)p_param;

			platform_thread.m_Alive = true;
			unsigned long ret = (unsigned long)platform_thread.mp_ThreadMain(platform_thread.mp_Param);
			platform_thread.m_Alive = false;

			return ret;
		}

		//------------------------------------------------------------------------
		int StartThread(void* p_os_thread_mem, ThreadMain p_thread_main, void* p_param)
		{
			PlatformThread& platform_thread = *(PlatformThread*)p_os_thread_mem;
			platform_thread.mp_ThreadMain = p_thread_main;
			platform_thread.mp_Param = p_param;

			platform_thread.m_Handle = ::CreateThread(NULL, 0, PlatformThreadMain, p_os_thread_mem, 0, NULL);

			return platform_thread.m_Handle ? GetThreadId(platform_thread.m_Handle) : 0;
		}

		//------------------------------------------------------------------------
		bool IsThreadAlive(const void* p_os_thread_mem)
		{
			const PlatformThread& platform_thread = *(PlatformThread*)p_os_thread_mem;
			return platform_thread.m_Alive;
		}

		//------------------------------------------------------------------------
		#ifndef USE_INTRINSIC
		MEMPRO_FORCEINLINE int64 ssInterlockedCompareExchange64(int64 volatile *dest, int64 exchange, int64 comperand)
		{
			__asm
			{
				lea esi, comperand;
				lea edi, exchange;
				mov eax, [esi];
				mov edx, 4[esi];
				mov ebx, [edi];
				mov ecx, 4[edi];
				mov esi, dest;
				lock CMPXCHG8B[esi];
			}
		}
		#endif

		//------------------------------------------------------------------------
		#ifndef USE_INTRINSIC
		MEMPRO_FORCEINLINE int64 ssInterlockedExchangeAdd64(__inout int64 volatile *Addend, __in int64 Value)
		{
			int64 Old;
			do
			{
				Old = *Addend;
			} while (ssInterlockedCompareExchange64(Addend, Old + Value, Old) != Old);
			return Old;
		}
		#endif

		//------------------------------------------------------------------------
		int64 MemProInterlockedCompareExchange(int64 volatile *dest, int64 exchange, int64 comperand)
		{
			MEMPRO_ASSERT((((int64)dest) & 7) == 0);

			#ifdef USE_INTRINSIC
				return _InterlockedCompareExchange64(dest, exchange, comperand);
			#else
				return ssInterlockedCompareExchange64(dest, exchange, comperand);
			#endif
		}

		//------------------------------------------------------------------------
		int64 MemProInterlockedExchangeAdd(int64 volatile *Addend, int64 Value)
		{
			MEMPRO_ASSERT((((int64)Addend) & 7) == 0);

			#ifdef USE_INTRINSIC
				return _InterlockedExchangeAdd64(Addend, Value);
			#else
				return ssInterlockedExchangeAdd64(Addend, Value);
			#endif
		}

		//------------------------------------------------------------------------
		void SwapEndian(unsigned int& value)
		{
			value = _byteswap_ulong(value);
		}

		//------------------------------------------------------------------------
		void SwapEndian(uint64& value)
		{
			value = _byteswap_uint64(value);
		}

		//------------------------------------------------------------------------
		void DebugBreak()
		{
			::DebugBreak();
		}

		//------------------------------------------------------------------------
		void* Alloc(int size)
		{
			#ifdef __UNREAL__
				LLM_SCOPED_PAUSE_TRACKING(ELLMAllocType::System);
			#endif

			return VirtualAlloc(NULL, size, MEM_COMMIT, PAGE_READWRITE);
		}

		//------------------------------------------------------------------------
		void Free(void* p, int size)
		{
			#ifdef __UNREAL__
				LLM_SCOPED_PAUSE_TRACKING(ELLMAllocType::System);
			#endif

			VirtualFree(p, 0, MEM_RELEASE);
		}

		//------------------------------------------------------------------------
		#pragma warning(push)
		#pragma warning(disable:6322) // warning C6322: Empty _except block.
		void SetThreadName(unsigned int thread_id, const char* p_name)
		{
			 // see http://msdn.microsoft.com/en-us/library/xcb2z8hs.aspx
			const unsigned int MS_VC_EXCEPTION=0x406D1388;

			struct THREADNAME_INFO
			{
				unsigned int dwType;		// Must be 0x1000.
				LPCSTR szName;		// Pointer to name (in user addr space).
				unsigned int dwThreadID;	// Thread ID (-1=caller thread).
				unsigned int dwFlags;		// Reserved for future use, must be zero.
			};

			// on the xbox setting thread names messes up the XDK COM API that UnrealConsole uses so check to see if they have been
			// explicitly enabled
			Sleep(10);
			THREADNAME_INFO ThreadNameInfo;
			ThreadNameInfo.dwType		= 0x1000;
			ThreadNameInfo.szName		= p_name;
			ThreadNameInfo.dwThreadID	= thread_id;
			ThreadNameInfo.dwFlags		= 0;

			__try
			{
				RaiseException( MS_VC_EXCEPTION, 0, sizeof(ThreadNameInfo)/sizeof(ULONG_PTR), (ULONG_PTR*)&ThreadNameInfo );
			}
			__except(EXCEPTION_EXECUTE_HANDLER)
			{
			}
		}
		#pragma warning(pop)

		//------------------------------------------------------------------------
		void Sleep(int ms)
		{
			::Sleep(ms);
		}

		//------------------------------------------------------------------------
		void GetVirtualMemStats(size_t& reserved, size_t& committed)
		{
			MEMORY_BASIC_INFORMATION info;
			memset(&info, 0, sizeof(info));

			uint64 addr = 0;
			reserved = 0;
			committed = 0;

			HANDLE process = GetCurrentProcess();

			bool started = false;

			while(addr < MEMPRO_MAX_ADDRESS)
			{
				uint64 last_addr = addr;

				if(VirtualQueryEx(process, (void*)addr, &info, sizeof(info)) != 0)
				{
					switch(info.State)
					{
						case MEM_RESERVE: reserved += info.RegionSize; break;
						case MEM_COMMIT: committed += info.RegionSize; break;
					}

					addr += info.RegionSize;

					started = true;
				}
				else
				{
					if(started)
						break;

					addr = (addr & (~((size_t)MEMPRO_PAGE_SIZE -1))) + MEMPRO_PAGE_SIZE;
				}

				if(addr < last_addr)		// handle wrap around
					break;
			}

			reserved += committed;
		}

		//------------------------------------------------------------------------
		bool GetExtraModuleInfo(
			int64 ModuleBase,
			int& age,
			void* p_guid,
			int guid_size,
			char* p_pdb_filename,
			int pdb_filename_size)
		{
			IMAGE_DOS_HEADER* p_dos_header = (IMAGE_DOS_HEADER*)ModuleBase;
			IMAGE_NT_HEADERS* p_nt_header = (IMAGE_NT_HEADERS*)((char*)ModuleBase + p_dos_header->e_lfanew);
			IMAGE_OPTIONAL_HEADER& optional_header = p_nt_header->OptionalHeader;
			IMAGE_DATA_DIRECTORY& image_data_directory = optional_header.DataDirectory[IMAGE_DIRECTORY_ENTRY_DEBUG];
			IMAGE_DEBUG_DIRECTORY* p_debug_info_array = (IMAGE_DEBUG_DIRECTORY*)(ModuleBase + image_data_directory.VirtualAddress);
			int count = image_data_directory.Size / sizeof(IMAGE_DEBUG_DIRECTORY);
			for(int i=0; i<count; ++i)
			{
				if(p_debug_info_array[i].Type == IMAGE_DEBUG_TYPE_CODEVIEW)
				{
					char* p_cv_data = (char*)(ModuleBase + p_debug_info_array[i].AddressOfRawData);
					if(strncmp(p_cv_data, "RSDS", 4) == 0)
					{
						CV_INFO_PDB70* p_cv_info = (CV_INFO_PDB70*)p_cv_data;
						age = p_cv_info->Age;
						MEMPRO_ASSERT(guid_size >= sizeof(p_cv_info->Signature));
						memcpy(p_guid, &p_cv_info->Signature, sizeof(p_cv_info->Signature));
						MEMPRO_ASSERT(pdb_filename_size >= sizeof(p_cv_info->PdbFileName));
						memcpy(p_pdb_filename, p_cv_info->PdbFileName, sizeof(p_cv_info->PdbFileName));
						return true;									// returning here
					}
					else if(strncmp(p_cv_data, "NB10", 4) == 0)
					{
						CV_INFO_PDB20* p_cv_info = (CV_INFO_PDB20*)p_cv_data;
						age = p_cv_info->Age;
						MEMPRO_ASSERT(guid_size >= sizeof(p_cv_info->Signature));
						memcpy(p_guid, &p_cv_info->Signature, sizeof(p_cv_info->Signature));
						MEMPRO_ASSERT(pdb_filename_size >= sizeof(p_cv_info->PdbFileName));
						memcpy(p_pdb_filename, p_cv_info->PdbFileName, sizeof(p_cv_info->PdbFileName));
						return true;									// returning here
					}
				}
			}

			return false;
		}

		//------------------------------------------------------------------------
		struct EnumModulesContext
		{
			EnumerateLoadedModulesCallbackFunction mp_CallbackFunction;
			void* mp_Context;
			int m_ModuleCount;
		};

		//------------------------------------------------------------------------
		#ifdef ENUMERATE_ALL_MODULES
		#if !defined(_IMAGEHLP_SOURCE_) && defined(_IMAGEHLP64)
		// depending on your platform you may need to change PCSTR to PSTR for ModuleName
		BOOL CALLBACK EnumerateLoadedModulesCallback(__in PCSTR ModuleName,__in DWORD64 ModuleBase,__in ULONG ModuleSize,__in_opt PVOID UserContext)
		#else
		BOOL CALLBACK EnumerateLoadedModulesCallback(__in PCSTR ModuleName,__in ULONG ModuleBase,__in ULONG ModuleSize,__in_opt PVOID UserContext)
		#endif
		{
			if(!UserContext)
				return false;

			EnumModulesContext* p_context = (EnumModulesContext*)UserContext;

			p_context->mp_CallbackFunction(ModuleBase, ModuleName, p_context->mp_Context);

			++p_context->m_ModuleCount;

			return true;
		}
		#endif

		//------------------------------------------------------------------------
		void MemProEnumerateLoadedModules(EnumerateLoadedModulesCallbackFunction p_callback_function, void* p_context)
		{
			EnumModulesContext context;
			context.mp_CallbackFunction = p_callback_function;
			context.mp_Context = p_context;
			context.m_ModuleCount = 0;

			#ifdef ENUMERATE_ALL_MODULES
				#ifdef MEMPRO64
					EnumerateLoadedModules64(GetCurrentProcess(), EnumerateLoadedModulesCallback, &context);
				#else
					EnumerateLoadedModules(GetCurrentProcess(), EnumerateLoadedModulesCallback, &context);
				#endif
			#endif

			// if ENUMERATE_ALL_MODULES is disabled or enumeration failed for some reason, fall back
			// to getting the base address for the main module. This will always for for all platforms.
			if (context.m_ModuleCount == 0)
			{
				static int module = 0;
				HMODULE module_handle = 0;
				GetModuleHandleEx(GET_MODULE_HANDLE_EX_FLAG_FROM_ADDRESS, (LPCTSTR)&module, &module_handle);

				int64 module_base = (int64)module_handle;

				TCHAR tchar_filename[MAX_PATH] = { 0 };
				GetModuleFileName(NULL, tchar_filename, MAX_PATH);

				char char_filename[MAX_PATH];

				#ifdef UNICODE
					size_t chars_converted = 0;
					wcstombs_s(&chars_converted, char_filename, tchar_filename, MAX_PATH);
				#else
					strcpy_s(char_filename, tchar_filename);
				#endif

				p_callback_function(module_base, char_filename, p_context);
			}
		}

		//------------------------------------------------------------------------
		void DebugWrite(const char* p_message)
		{
			OutputDebugStringA(p_message);
		}

		//------------------------------------------------------------------------
		void MemCpy(void* p_dest, int dest_size, const void* p_source, int source_size)
		{
			memcpy_s(p_dest, dest_size, p_source, source_size);
		}

		//------------------------------------------------------------------------
		void SPrintF(char* p_dest, int dest_size, const char* p_format, const char* p_str)
		{
			sprintf_s(p_dest, dest_size, p_format, p_str);
		}

		//------------------------------------------------------------------------
		void MemProCreateFile(void* p_os_file_mem, int os_file_mem_size)
		{
			MEMPRO_ASSERT(os_file_mem_size >= sizeof(FILE*));
			FILE*& p_file = *(FILE**)p_os_file_mem;
			p_file = NULL;
		}

		//------------------------------------------------------------------------
		void DestroyFile(void* p_os_file_mem)
		{
			FILE*& p_file = *(FILE**)p_os_file_mem;
			p_file = NULL;
		}

		//------------------------------------------------------------------------
		bool OpenFileForWrite(void* p_os_file_mem, const char* p_filename)
		{
			FILE*& p_file = *(FILE**)p_os_file_mem;
			fopen_s(&p_file, p_filename, "wb");
			MEMPRO_ASSERT(p_file);
			return p_file != NULL;
		}

		//------------------------------------------------------------------------
		void CloseFile(void* p_os_file_mem)
		{
			FILE*& p_file = *(FILE**)p_os_file_mem;
			fclose(p_file);
			p_file = NULL;
		}

		//------------------------------------------------------------------------
		void FlushFile(void* p_os_file_mem)
		{
			FILE*& p_file = *(FILE**)p_os_file_mem;
			fflush(p_file);
		}

		//------------------------------------------------------------------------
		bool WriteFile(void* p_os_file_mem, const void* p_data, int size)
		{
			FILE*& p_file = *(FILE**)p_os_file_mem;
			return fwrite(p_data, size, 1, p_file) == 1;
		}

		//------------------------------------------------------------------------
		#ifdef MEMPRO_WRITE_DUMP
		void GetDumpFilename(char* p_filename, int max_length)
		{
			#ifdef __UNREAL__
				const FString Directory = FPaths::ProfilingDir() / "MemPro";
				IFileManager::Get().MakeDirectory(*Directory, true);

				const FDateTime FileDate = FDateTime::Now();
				FString Filename = FString::Printf(TEXT("%s/MemPro_%s.mempro_dump"), *Directory, *FileDate.ToString());
				FPaths::NormalizeFilename(Filename);

				Filename = IFileManager::Get().ConvertToAbsolutePathForExternalAppForWrite(*Filename); //@EPIC: Fix bad path for locally compiled and cooked builds

				Platform::MemCpy(p_filename, max_length, TCHAR_TO_ANSI(*Filename), (int)(strlen(TCHAR_TO_ANSI(*Filename)) + 1));
			#else
				Platform::MemCpy(p_filename, max_length, MEMPRO_WRITE_DUMP, (int)(strlen(MEMPRO_WRITE_DUMP) + 1));
			#endif
		}
		#endif
	}
}

//------------------------------------------------------------------------
#pragma warning(pop)

#elif defined(MEMPRO_UNIX_BASED_PLATFORM)

//------------------------------------------------------------------------
#include <execinfo.h>
#include <inttypes.h>
#include <stdlib.h>
#include <string.h>
#include <pthread.h>
#include <sys/time.h>
#include <sys/socket.h>
#include <netinet/in.h>
#include <unistd.h>
#include <stdio.h>

#ifndef MEMPRO_PLATFORM_APPLE
	#include <link.h>
#endif

//------------------------------------------------------------------------
#include <pthread.h>

//------------------------------------------------------------------------
namespace MemPro
{
	//------------------------------------------------------------------------
	typedef sockaddr_in SOCKADDR_IN;

	//------------------------------------------------------------------------
	const int g_StackTraceSize = 128;

	//------------------------------------------------------------------------
	typedef int SOCKET;
	typedef int DWORD;
	enum SocketValues { INVALID_SOCKET = -1 };
	#ifndef UINT_MAX
		enum MaxValues { UINT_MAX = 0xffffffff };
	#endif
	#define _T(s) s
	enum SocketErrorCodes { SOCKET_ERROR = -1 };
	enum SystemDefines { MAX_PATH = 256 };

	//------------------------------------------------------------------------
	namespace GenericPlatform
	{
		//------------------------------------------------------------------------
		pthread_mutex_t& GetOSLock(void* p_os_lock_mem)
		{
			return *(pthread_mutex_t*)p_os_lock_mem;
		}

		//------------------------------------------------------------------------
		void CreateLock(void* p_os_lock_mem, int os_lock_mem_size)
		{
			MEMPRO_ASSERT((size_t)os_lock_mem_size >= sizeof(pthread_mutex_t));

			new (p_os_lock_mem)pthread_mutex_t();

			pthread_mutexattr_t attr;
			pthread_mutexattr_init(&attr);
			pthread_mutexattr_settype(&attr, PTHREAD_MUTEX_RECURSIVE);
			pthread_mutex_init(&GetOSLock(p_os_lock_mem), &attr);
		}

		//------------------------------------------------------------------------
		void DestroyLock(void* p_os_lock_mem)
		{
			pthread_mutex_t& mutex = GetOSLock(p_os_lock_mem);
			pthread_mutex_destroy(&mutex);
			mutex.~pthread_mutex_t();
		}

		//------------------------------------------------------------------------
		void TakeLock(void* p_os_lock_mem)
		{
			pthread_mutex_lock(&GetOSLock(p_os_lock_mem));
		}

		//------------------------------------------------------------------------
		void ReleaseLock(void* p_os_lock_mem)
		{
			pthread_mutex_unlock(&GetOSLock(p_os_lock_mem));
		}

		//------------------------------------------------------------------------
		#ifndef MEMPRO_WRITE_DUMP
		void UninitialiseSockets()
		{
			// do nothing
		}
		#endif

		//------------------------------------------------------------------------
		#ifndef MEMPRO_WRITE_DUMP
		void CreateSocket(void* p_os_socket_mem, int os_socket_mem_size)
		{
			MEMPRO_ASSERT(os_socket_mem_size >= sizeof(SOCKET));
			*(SOCKET*)p_os_socket_mem = INVALID_SOCKET;
		}
		#endif

		//------------------------------------------------------------------------
		#ifndef MEMPRO_WRITE_DUMP
		bool IsValidSocket(const void* p_os_socket_mem)
		{
			return *(const SOCKET*)p_os_socket_mem != INVALID_SOCKET;
		}
		#endif

		//------------------------------------------------------------------------
		#ifndef MEMPRO_WRITE_DUMP
		void HandleError()
		{
			MEMPRO_ASSERT(false);
		}
		#endif

		//------------------------------------------------------------------------
		#ifndef MEMPRO_WRITE_DUMP
		void Disconnect(void* p_os_socket_mem)
		{
			SOCKET& socket = *(SOCKET*)p_os_socket_mem;

			if (socket != INVALID_SOCKET)
			{
				if (shutdown(socket, SHUT_RDWR) == SOCKET_ERROR)
					HandleError();

				// loop until the socket is closed to ensure all data is sent
				unsigned int buffer = 0;
				size_t ret = 0;
				do { ret = recv(socket, (char*)&buffer, sizeof(buffer), 0); } while (ret != 0 && ret != (size_t)SOCKET_ERROR);

				close(socket);

				socket = INVALID_SOCKET;
			}
		}
		#endif

		//------------------------------------------------------------------------
		#ifndef MEMPRO_WRITE_DUMP
		bool StartListening(void* p_os_socket_mem)
		{
			SOCKET& socket = *(SOCKET*)p_os_socket_mem;

			MEMPRO_ASSERT(socket != INVALID_SOCKET);

			if (listen(socket, SOMAXCONN) == SOCKET_ERROR)
			{
				HandleError();
				return false;
			}
			return true;
		}
		#endif

		//------------------------------------------------------------------------
		#ifndef MEMPRO_WRITE_DUMP
		bool BindSocket(void* p_os_socket_mem, const char* p_port)
		{
			SOCKET& socket = *(SOCKET*)p_os_socket_mem;

			MEMPRO_ASSERT(socket == INVALID_SOCKET);

			socket = ::socket(
				AF_INET,
				SOCK_STREAM,
				IPPROTO_TCP);

			if (socket == INVALID_SOCKET)
			{
				HandleError();
				return false;
			}

			// Setup the TCP listening socket
			// Bind to INADDR_ANY
			SOCKADDR_IN sa;
			sa.sin_family = AF_INET;
			sa.sin_addr.s_addr = INADDR_ANY;
			int iport = atoi(p_port);
			sa.sin_port = htons(iport);
			int result = ::bind(socket, (const sockaddr*)(&sa), sizeof(SOCKADDR_IN));

			if (result == SOCKET_ERROR)
			{
				HandleError();
				Disconnect(p_os_socket_mem);
				return false;
			}

			return true;
		}
		#endif

		//------------------------------------------------------------------------
		#ifndef MEMPRO_WRITE_DUMP
		bool AcceptSocket(void* p_os_socket_mem, void* p_client_os_socket_mem)
		{
			SOCKET& socket = *(SOCKET*)p_os_socket_mem;
			SOCKET& client_socket = *(SOCKET*)p_client_os_socket_mem;

			MEMPRO_ASSERT(client_socket == INVALID_SOCKET);
			client_socket = accept(socket, NULL, NULL);

			return client_socket != INVALID_SOCKET;
		}
		#endif

		//------------------------------------------------------------------------
		#ifndef MEMPRO_WRITE_DUMP
		bool SocketSend(void* p_os_socket_mem, void* p_buffer, int size)
		{
			SOCKET& socket = *(SOCKET*)p_os_socket_mem;

			int bytes_to_send = size;
			while (bytes_to_send != 0)
			{
				int bytes_sent = (int)send(socket, (char*)p_buffer, bytes_to_send, 0);

				if (bytes_sent == SOCKET_ERROR)
				{
					HandleError();
					Disconnect(p_os_socket_mem);
					return false;
				}
				p_buffer = (char*)p_buffer + bytes_sent;
				bytes_to_send -= bytes_sent;
			}

			return true;
		}
		#endif

		//------------------------------------------------------------------------
		#ifndef MEMPRO_WRITE_DUMP
		int SocketReceive(void* p_os_socket_mem, void* p_buffer, int size)
		{
			SOCKET& socket = *(SOCKET*)p_os_socket_mem;

			int total_bytes_received = 0;
			while (size)
			{
				int bytes_received = (int)recv(socket, (char*)p_buffer, size, 0);

				total_bytes_received += bytes_received;

				if (bytes_received == 0)
				{
					Disconnect(p_os_socket_mem);
					return bytes_received;
				}
				else if (bytes_received == SOCKET_ERROR)
				{
					HandleError();
					Disconnect(p_os_socket_mem);
					return bytes_received;
				}

				size -= bytes_received;
				p_buffer = (char*)p_buffer + bytes_received;
			}

			return total_bytes_received;
		}
		#endif

		//------------------------------------------------------------------------
		struct PlatformEvent
		{
			mutable pthread_cond_t  m_Cond;
			mutable pthread_mutex_t m_Mutex;
			mutable volatile bool m_Signalled;
			bool m_AutoReset;
		};

		//------------------------------------------------------------------------
		void SetEvent(void* p_os_event_mem);
		
		//------------------------------------------------------------------------
		void MemProCreateEvent(
			void* p_os_event_mem,
			int os_event_mem_size,
			bool initial_state,
			bool auto_reset)
		{
			MEMPRO_ASSERT(os_event_mem_size >= sizeof(PlatformEvent));

			PlatformEvent& platform_event = *(PlatformEvent*)p_os_event_mem;

			pthread_cond_init(&platform_event.m_Cond, NULL);
			pthread_mutex_init(&platform_event.m_Mutex, NULL);
			platform_event.m_Signalled = false;
			platform_event.m_AutoReset = auto_reset;

			if (initial_state)
				SetEvent(p_os_event_mem);
		}

		//------------------------------------------------------------------------
		void DestroyEvent(void* p_os_event_mem)
		{
			PlatformEvent& platform_event = *(PlatformEvent*)p_os_event_mem;

			pthread_mutex_destroy(&platform_event.m_Mutex);
			pthread_cond_destroy(&platform_event.m_Cond);
		}

		//------------------------------------------------------------------------
		void SetEvent(void* p_os_event_mem)
		{
			PlatformEvent& platform_event = *(PlatformEvent*)p_os_event_mem;

			pthread_mutex_lock(&platform_event.m_Mutex);
			platform_event.m_Signalled = true;
			pthread_mutex_unlock(&platform_event.m_Mutex);
			pthread_cond_signal(&platform_event.m_Cond);
		}

		//------------------------------------------------------------------------
		void ResetEvent(void* p_os_event_mem)
		{
			PlatformEvent& platform_event = *(PlatformEvent*)p_os_event_mem;

			pthread_mutex_lock(&platform_event.m_Mutex);
			platform_event.m_Signalled = false;
			pthread_mutex_unlock(&platform_event.m_Mutex);
		}

		//------------------------------------------------------------------------
		int WaitEvent(void* p_os_event_mem, int timeout)
		{
			PlatformEvent& platform_event = *(PlatformEvent*)p_os_event_mem;

			pthread_mutex_lock(&platform_event.m_Mutex);

			if (platform_event.m_Signalled)
			{
				platform_event.m_Signalled = false;
				pthread_mutex_unlock(&platform_event.m_Mutex);
				return true;
			}

			if (timeout == -1)
			{
				while (!platform_event.m_Signalled)
					pthread_cond_wait(&platform_event.m_Cond, &platform_event.m_Mutex);

				if (!platform_event.m_AutoReset)
					platform_event.m_Signalled = false;

				pthread_mutex_unlock(&platform_event.m_Mutex);

				return true;
			}
			else
			{
				timeval curr;
				gettimeofday(&curr, NULL);

				timespec time;
				time.tv_sec = curr.tv_sec + timeout / 1000;
				time.tv_nsec = (curr.tv_usec * 1000) + ((timeout % 1000) * 1000000);

				pthread_cond_timedwait(&platform_event.m_Cond, &platform_event.m_Mutex, &time);

				if (platform_event.m_Signalled)
				{
					if (!platform_event.m_AutoReset)
						platform_event.m_Signalled = false;

					pthread_mutex_unlock(&platform_event.m_Mutex);
					return true;
				}

				pthread_mutex_unlock(&platform_event.m_Mutex);
				return false;
			}
		}

		//------------------------------------------------------------------------
		struct PlatformThread
		{
			pthread_t m_Handle;
			bool m_Alive;
			ThreadMain mp_ThreadMain;
			void* mp_Param;
		};

		//------------------------------------------------------------------------
		void CreateThread(void* p_os_thread_mem, int os_thread_mem_size)
		{
			MEMPRO_ASSERT(os_thread_mem_size >= sizeof(PlatformThread));
			new (p_os_thread_mem)PlatformThread();
			PlatformThread& platform_thread = *(PlatformThread*)p_os_thread_mem;
			platform_thread.m_Alive = false;
			platform_thread.mp_ThreadMain = NULL;
			platform_thread.mp_Param = NULL;
		}

		//------------------------------------------------------------------------
		void DestroyThread(void* p_os_thread_mem)
		{
			PlatformThread& platform_thread = *(PlatformThread*)p_os_thread_mem;
			platform_thread.~PlatformThread();
		}

		//------------------------------------------------------------------------
		void* PlatformThreadMain(void* p_param)
		{
			PlatformThread& platform_thread = *(PlatformThread*)p_param;

			platform_thread.m_Alive = true;
			unsigned long ret = (unsigned long)platform_thread.mp_ThreadMain(platform_thread.mp_Param);
			platform_thread.m_Alive = false;

			return NULL;
		}

		//------------------------------------------------------------------------
		int StartThread(void* p_os_thread_mem, ThreadMain p_thread_main, void* p_param)
		{
			PlatformThread& platform_thread = *(PlatformThread*)p_os_thread_mem;
			platform_thread.mp_ThreadMain = p_thread_main;
			platform_thread.mp_Param = p_param;

			pthread_create(&platform_thread.m_Handle, NULL, PlatformThreadMain, p_os_thread_mem);

			return 0;
		}

		//------------------------------------------------------------------------
		bool IsThreadAlive(const void* p_os_thread_mem)
		{
			const PlatformThread& platform_thread = *(PlatformThread*)p_os_thread_mem;
			return platform_thread.m_Alive;
		}

		//------------------------------------------------------------------------
		// global lock for all CAS instructions. This is Ok because CAS is only used by RingBuffer
		CriticalSection g_CASCritSec;

		//------------------------------------------------------------------------
		int64 MemProInterlockedCompareExchange(int64 volatile *dest, int64 exchange, int64 comperand)
		{
			g_CASCritSec.Enter();
			int64 old_value = *dest;
			if (*dest == comperand)
				*dest = exchange;
			g_CASCritSec.Leave();
			return old_value;
		}

		//------------------------------------------------------------------------
		int64 MemProInterlockedExchangeAdd(int64 volatile *Addend, int64 Value)
		{
			g_CASCritSec.Enter();
			int64 old_value = *Addend;
			*Addend += Value;
			g_CASCritSec.Leave();
			return old_value;
		}

		//------------------------------------------------------------------------
		void SwapEndian(unsigned int& value)
		{
#ifdef MEMPRO_PLATFORM_APPLE
			value = __builtin_bswap32(value);
#else
			value = __bswap_32(value);
#endif
		}

		//------------------------------------------------------------------------
		void SwapEndian(uint64& value)
		{
#ifdef MEMPRO_PLATFORM_APPLE
			value = __builtin_bswap64(value);
#else
			value = __bswap_64(value);
#endif
		}

		//------------------------------------------------------------------------
		void DebugBreak()
		{
			__builtin_trap();
		}

		//------------------------------------------------------------------------
		void* Alloc(int size)
		{
			#ifdef __UNREAL__
				LLM_SCOPED_PAUSE_TRACKING(ELLMAllocType::System);
			#endif

			return malloc(size);
		}

		//------------------------------------------------------------------------
		void Free(void* p, int size)
		{
			#ifdef __UNREAL__
				LLM_SCOPED_PAUSE_TRACKING(ELLMAllocType::System);
			#endif

			free(p);
		}

		//------------------------------------------------------------------------
		void SetThreadName(unsigned int thread_id, const char* p_name)
		{
			// not supported on this platform
		}

		//------------------------------------------------------------------------
		void Sleep(int ms)
		{
			usleep(ms * 1000);
		}

		//------------------------------------------------------------------------
		void SendPageState(bool, SendPageStateFunction, void*)
		{
			// not supported on this platform
		}

		//------------------------------------------------------------------------
		void GetVirtualMemStats(size_t& reserved, size_t& committed)
		{
			// not supported on this platform
			reserved = 0;
			committed = 0;
		}

		//------------------------------------------------------------------------
		bool GetExtraModuleInfo(int64, int&, void*, int, char*, int)
		{
			// not supported on this platform
			return false;
		}

		//------------------------------------------------------------------------
		struct EnumModulesContext
		{
			EnumerateLoadedModulesCallbackFunction mp_CallbackFunction;
			void* mp_Context;
			int m_ModuleCount;
		};

		//------------------------------------------------------------------------
		#ifdef ENUMERATE_ALL_MODULES
		int EnumerateLoadedModulesCallback(struct dl_phdr_info* info, size_t size, void* data)
		{
			EnumModulesContext* p_context = (EnumModulesContext*)data;

			int64 module_base = 0;
			for (int j = 0; j < info->dlpi_phnum; j++)
			{
				if (info->dlpi_phdr[j].p_type == PT_LOAD)
				{
					module_base = info->dlpi_addr + info->dlpi_phdr[j].p_vaddr;
					break;
				}
			}

			if(!p_context->m_ModuleCount)
			{
				int64 base_address = 0xabcdefabcdef1LL;	// lookup function marker

				// get the module name
				char arg1[20];
				char char_filename[MAX_PATH];
				sprintf(arg1, "/proc/%d/exe", getpid());
				memset(char_filename, 0, MAX_PATH);
				readlink(arg1, char_filename, MAX_PATH-1);
				
				p_context->mp_CallbackFunction(base_address, char_filename, p_context->mp_Context);
			}
			else
			{
				p_context->mp_CallbackFunction(module_base, info->dlpi_name, p_context->mp_Context);
			}

			++p_context->m_ModuleCount;

			return 0;
		}
		#endif

		//------------------------------------------------------------------------
		void MemProEnumerateLoadedModules(
			EnumerateLoadedModulesCallbackFunction p_callback_function,
			void* p_context)
		{
			EnumModulesContext context;
			context.mp_CallbackFunction = p_callback_function;
			context.mp_Context = p_context;
			context.m_ModuleCount = 0;

			#ifdef ENUMERATE_ALL_MODULES
				dl_iterate_phdr(EnumerateLoadedModulesCallback, &context);
			#endif

			// if ENUMERATE_ALL_MODULES is disabled or enumeration failed for some reason, fall back
			// to getting the base address for the main module. This will always for for all platforms.
			if (!context.m_ModuleCount)
			{
				// send the module base address
				int64 module_base = (int64)0xabcdefabcdef1LL;		// use the address of the BaseAddressLookupFunction function so that we can work it out later

				// send the module name
				char char_filename[MAX_PATH];

				// get the module name
				char arg1[20];
				sprintf(arg1, "/proc/%d/exe", getpid());
				memset(char_filename, 0, MAX_PATH);
				readlink(arg1, char_filename, MAX_PATH-1);

				p_callback_function(module_base, char_filename, p_context);
			}
		}

		//------------------------------------------------------------------------
		void DebugWrite(const char* p_message)
		{
			printf("%s", p_message);
		}

		//------------------------------------------------------------------------
		void MemCpy(void* p_dest, int dest_size, const void* p_source, int source_size)
		{
			MEMPRO_ASSERT(dest_size >= source_size);
			memcpy(p_dest, p_source, source_size);
		}

		//------------------------------------------------------------------------
		void SPrintF(char* p_dest, int, const char* p_format, const char* p_str)
		{
			sprintf(p_dest, p_format, p_str);
		}

		//------------------------------------------------------------------------
		void MemProCreateFile(void* p_os_file_mem, int os_file_mem_size)
		{
			MEMPRO_ASSERT(os_file_mem_size >= sizeof(FILE*));
			FILE*& p_file = *(FILE**)p_os_file_mem;
			p_file = NULL;
		}

		//------------------------------------------------------------------------
		void DestroyFile(void* p_os_file_mem)
		{
			FILE*& p_file = *(FILE**)p_os_file_mem;
			p_file = NULL;
		}

		//------------------------------------------------------------------------
		bool OpenFileForWrite(void* p_os_file_mem, const char* p_filename)
		{
			FILE*& p_file = *(FILE**)p_os_file_mem;
			p_file = fopen(p_filename, "wb");
			MEMPRO_ASSERT(p_file);
			return p_file != NULL;
		}

		//------------------------------------------------------------------------
		void CloseFile(void* p_os_file_mem)
		{
			FILE*& p_file = *(FILE**)p_os_file_mem;
			fclose(p_file);
			p_file = NULL;
		}

		//------------------------------------------------------------------------
		void FlushFile(void* p_os_file_mem)
		{
			FILE*& p_file = *(FILE**)p_os_file_mem;
			fflush(p_file);
		}

		//------------------------------------------------------------------------
		bool WriteFile(void* p_os_file_mem, const void* p_data, int size)
		{
			FILE*& p_file = *(FILE**)p_os_file_mem;
			return fwrite(p_data, size, 1, p_file) == 1;
		}

		//------------------------------------------------------------------------
		#ifdef MEMPRO_WRITE_DUMP
		void GetDumpFilename(char* p_filename, int max_length)
		{
			#ifdef __UNREAL__
				const FString Directory = FPaths::ProfilingDir() / "MemPro";
				IFileManager::Get().MakeDirectory(*Directory, true);

				const FDateTime FileDate = FDateTime::Now();
				FString Filename = FString::Printf(TEXT("%s/MemPro_%s.mempro_dump"), *Directory, *FileDate.ToString());
				FPaths::NormalizeFilename(Filename);

				Platform::MemCpy(p_filename, max_length, TCHAR_TO_ANSI(*Filename), (int)(strlen(TCHAR_TO_ANSI(*Filename)) + 1));
			#else
				Platform::MemCpy(p_filename, max_length, MEMPRO_WRITE_DUMP, strlen(MEMPRO_WRITE_DUMP) + 1);
			#endif
		}
		#endif
	}
}

#endif

//------------------------------------------------------------------------
#if defined(MEMPRO_PLATFORM_WIN)

//------------------------------------------------------------------------
namespace MemPro
{
	//------------------------------------------------------------------------
	#if (NTDDI_VERSION > NTDDI_WINXP)
		const int g_StackTraceSize = 128;
	#else
		const int g_StackTraceSize = 62;
	#endif

	//------------------------------------------------------------------------
	void Platform::CreateLock(void* p_os_lock_mem, int os_lock_mem_size)
	{
		GenericPlatform::CreateLock(p_os_lock_mem, os_lock_mem_size);
	}

	//------------------------------------------------------------------------
	void Platform::DestroyLock(void* p_os_lock_mem)
	{
		GenericPlatform::DestroyLock(p_os_lock_mem);
	}

	//------------------------------------------------------------------------
	void Platform::TakeLock(void* p_os_lock_mem)
	{
		GenericPlatform::TakeLock(p_os_lock_mem);
	}

	//------------------------------------------------------------------------
	void Platform::ReleaseLock(void* p_os_lock_mem)
	{
		GenericPlatform::ReleaseLock(p_os_lock_mem);
	}

	//------------------------------------------------------------------------
	#ifndef MEMPRO_WRITE_DUMP
	void Platform::UninitialiseSockets()
	{
		GenericPlatform::UninitialiseSockets();
	}
	#endif

	//------------------------------------------------------------------------
	#ifndef MEMPRO_WRITE_DUMP
	void Platform::CreateSocket(void* p_os_socket_mem, int os_socket_mem_size)
	{
		GenericPlatform::CreateSocket(p_os_socket_mem, os_socket_mem_size);
	}
	#endif

	//------------------------------------------------------------------------
	#ifndef MEMPRO_WRITE_DUMP
	bool Platform::IsValidSocket(const void* p_os_socket_mem)
	{
		return GenericPlatform::IsValidSocket(p_os_socket_mem);
	}
	#endif

	//------------------------------------------------------------------------
	#ifndef MEMPRO_WRITE_DUMP
	void Platform::Disconnect(void* p_os_socket_mem)
	{
		GenericPlatform::Disconnect(p_os_socket_mem);
	}
	#endif

	//------------------------------------------------------------------------
	#ifndef MEMPRO_WRITE_DUMP
	bool Platform::StartListening(void* p_os_socket_mem)
	{
		return GenericPlatform::StartListening(p_os_socket_mem);
	}
	#endif

	//------------------------------------------------------------------------
	#ifndef MEMPRO_WRITE_DUMP
	bool Platform::BindSocket(void* p_os_socket_mem, const char* p_port)
	{
		return GenericPlatform::BindSocket(p_os_socket_mem, p_port);
	}
	#endif

	//------------------------------------------------------------------------
	#ifndef MEMPRO_WRITE_DUMP
	bool Platform::AcceptSocket(void* p_os_socket_mem, void* p_client_os_socket_mem)
	{
		return GenericPlatform::AcceptSocket(p_os_socket_mem, p_client_os_socket_mem);
	}
	#endif

	//------------------------------------------------------------------------
	#ifndef MEMPRO_WRITE_DUMP
	bool Platform::SocketSend(void* p_os_socket_mem, void* p_buffer, int size)
	{
		return GenericPlatform::SocketSend(p_os_socket_mem, p_buffer, size);
	}
	#endif

	//------------------------------------------------------------------------
	#ifndef MEMPRO_WRITE_DUMP
	int Platform::SocketReceive(void* p_os_socket_mem, void* p_buffer, int size)
	{
		return GenericPlatform::SocketReceive(p_os_socket_mem, p_buffer, size);
	}
	#endif

	//------------------------------------------------------------------------
	void Platform::MemProCreateEvent(
		void* p_os_event_mem,
		int os_event_mem_size,
		bool initial_state,
		bool auto_reset)
	{
		GenericPlatform::MemProCreateEvent(
			p_os_event_mem,
			os_event_mem_size,
			initial_state,
			auto_reset);
	}

	//------------------------------------------------------------------------
	void Platform::DestroyEvent(void* p_os_event_mem)
	{
		GenericPlatform::DestroyEvent(p_os_event_mem);
	}

	//------------------------------------------------------------------------
	void Platform::SetEvent(void* p_os_event_mem)
	{
		GenericPlatform::SetEvent(p_os_event_mem);
	}

	//------------------------------------------------------------------------
	void Platform::ResetEvent(void* p_os_event_mem)
	{
		GenericPlatform::ResetEvent(p_os_event_mem);
	}

	//------------------------------------------------------------------------
	int Platform::WaitEvent(void* p_os_event_mem, int timeout)
	{
		return GenericPlatform::WaitEvent(p_os_event_mem, timeout);
	}

	//------------------------------------------------------------------------
	void Platform::CreateThread(void* p_os_thread_mem, int os_thread_mem_size)
	{
		GenericPlatform::CreateThread(p_os_thread_mem, os_thread_mem_size);
	}

	//------------------------------------------------------------------------
	void Platform::DestroyThread(void* p_os_thread_mem)
	{
		GenericPlatform::DestroyThread(p_os_thread_mem);
	}

	//------------------------------------------------------------------------
	int Platform::StartThread(void* p_os_thread_mem, ThreadMain p_thread_main, void* p_param)
	{
		return GenericPlatform::StartThread(p_os_thread_mem, p_thread_main, p_param);
	}

	//------------------------------------------------------------------------
	bool Platform::IsThreadAlive(const void* p_os_thread_mem)
	{
		return GenericPlatform::IsThreadAlive(p_os_thread_mem);
	}

	//------------------------------------------------------------------------
	int64 Platform::MemProInterlockedCompareExchange(int64 volatile *dest, int64 exchange, int64 comperand)
	{
		return GenericPlatform::MemProInterlockedCompareExchange(dest, exchange, comperand);
	}

	//------------------------------------------------------------------------
	int64 Platform::MemProInterlockedExchangeAdd(int64 volatile *Addend, int64 Value)
	{
		return GenericPlatform::MemProInterlockedExchangeAdd(Addend, Value);
	}

	//------------------------------------------------------------------------
	void Platform::SwapEndian(unsigned int& value)
	{
		GenericPlatform::SwapEndian(value);
	}

	//------------------------------------------------------------------------
	void Platform::SwapEndian(uint64& value)
	{
		GenericPlatform::SwapEndian(value);
	}

	//------------------------------------------------------------------------
	void Platform::DebugBreak()
	{
		GenericPlatform::DebugBreak();
	}

	//------------------------------------------------------------------------
	void* Platform::Alloc(int size)
	{
		return GenericPlatform::Alloc(size);
	}

	//------------------------------------------------------------------------
	void Platform::Free(void* p, int size)
	{
		GenericPlatform::Free(p, size);
	}

	//------------------------------------------------------------------------
	int64 Platform::GetHiResTimer()
	{
		#ifdef MEMPRO64
			#ifdef _M_ARM64
				// this will get CNTVCT_EL0, which appears to be the best replacement on arm
				return (int)_ReadStatusReg(ARM64_CNTVCT);
			#else
				return (int64)__rdtsc();
			#endif
		#else
			__asm
			{
				; Flush the pipeline
				XOR eax, eax
				CPUID
				; Get RDTSC counter in edx : eax
				RDTSC
			}
		#endif
	}

	//------------------------------------------------------------------------
	// very innacurate, but portable. Doesn't have to be exact for our needs
	int64 Platform::GetHiResTimerFrequency()
	{
		static bool calculated_frequency = false;
		static int64 frequency = 1;
		if (!calculated_frequency)
		{
			calculated_frequency = true;
			Platform::Sleep(100);
			int64 start = GetHiResTimer();
			Platform::Sleep(1000);
			int64 end = GetHiResTimer();
			frequency = end - start;
		}

		return frequency;
	}

	//------------------------------------------------------------------------
	void Platform::SetThreadName(unsigned int thread_id, const char* p_name)
	{
		GenericPlatform::SetThreadName(thread_id, p_name);
	}

	//------------------------------------------------------------------------
	void Platform::Sleep(int ms)
	{
		GenericPlatform::Sleep(ms);
	}

	//------------------------------------------------------------------------
	#ifdef USE_RTLVIRTUALUNWIND
	__declspec(noinline) VOID VirtualUnwindStackWalk(void** stack, int max_stack_size)
	{
		#ifndef MEMPRO64
			#error USE_RTLVIRTUALUNWIND only available on x64 builds. Please use a different stack walk function.
		#endif

		CONTEXT context;
		memset(&context, 0, sizeof(context));
		RtlCaptureContext(&context);

		UNWIND_HISTORY_TABLE unwind_history_table;
		RtlZeroMemory(&unwind_history_table, sizeof(UNWIND_HISTORY_TABLE));

		int frame = 0;
		for (; frame < max_stack_size-1; ++frame)
		{
			stack[frame] = (void*)context.Rip;

			ULONG64 image_base;
			PRUNTIME_FUNCTION runtime_function = RtlLookupFunctionEntry(context.Rip, &image_base, &unwind_history_table);

			if (!runtime_function)
			{
				// If we don't have a RUNTIME_FUNCTION, then we've encountered
				// a leaf function. Adjust the stack appropriately.
				context.Rip = (ULONG64)(*(PULONG64)context.Rsp);
				context.Rsp += 8;
			}
			else
			{
				// Otherwise, call upon RtlVirtualUnwind to execute the unwind for us.
				KNONVOLATILE_CONTEXT_POINTERS nv_context;
				RtlZeroMemory(&nv_context, sizeof(KNONVOLATILE_CONTEXT_POINTERS));

				PVOID handler_data;
				ULONG64 establisher_frame;

				RtlVirtualUnwind(
					0/*UNW_FLAG_NHANDLER*/,
					image_base,
					context.Rip,
					runtime_function,
					&context,
					&handler_data,
					&establisher_frame,
					&nv_context);
			}

			// If we reach an RIP of zero, this means that we've walked off the end
			// of the call stack and are done.
			if (!context.Rip)
				break;
		}

		stack[frame] = 0;
	}
	#endif

	//------------------------------------------------------------------------
	void Platform::GetStackTrace(void** stack, int& stack_size, unsigned int& hash)
	{
		#if defined(__UNREAL__)
			stack_size = FPlatformStackWalk::CaptureStackBackTrace( (uint64*)stack, g_StackTraceSize );
		#else
			#if defined(USE_STACKWALK64)

				#ifdef MEMPRO64
					#error USE_STACKWALK64 only works in x86 builds. Please use a different stack walk funtion.
				#endif

				// get the context
				CONTEXT context;
				memset(&context, 0, sizeof(context));
				RtlCaptureContext(&context);

				// setup the stack frame
				STACKFRAME64 stack_frame;
				memset(&stack_frame, 0, sizeof(stack_frame));
				stack_frame.AddrPC.Mode = AddrModeFlat;
				stack_frame.AddrFrame.Mode = AddrModeFlat;
				stack_frame.AddrStack.Mode = AddrModeFlat;
				#ifdef MEMPRO64
					DWORD machine = IMAGE_FILE_MACHINE_IA64;
					stack_frame.AddrPC.Offset = context.Rip;
					stack_frame.AddrFrame.Offset = context.Rsp;
					stack_frame.AddrStack.Offset = context.Rbp;
				#else
					DWORD machine = IMAGE_FILE_MACHINE_I386;
					stack_frame.AddrPC.Offset = context.Eip;
					stack_frame.AddrFrame.Offset = context.Ebp;
					stack_frame.AddrStack.Offset = context.Esp;
				#endif
				HANDLE thread = GetCurrentThread();

				static HANDLE process = GetCurrentProcess();

				stack_size = 0;
				while(StackWalk64(
					machine,
					process,
					thread,
					&stack_frame,
					&context,
					NULL,
					SymFunctionTableAccess64,
					SymGetModuleBase64,
					NULL) && stack_size < g_StackTraceSize)
				{
					void* p = (void*)(stack_frame.AddrPC.Offset);
					stack[stack_size++] = p;
				}
				hash = GetHash(stack, stack_size);
			#elif defined(USE_RTLVIRTUALUNWIND)
				MemPro::VirtualUnwindStackWalk(stack, g_StackTraceSize);
				hash = GetHashAndStackSize(stack, stack_size);
			#elif defined(USE_RTLCAPTURESTACKBACKTRACE)
				MemPro::RTLCaptureStackBackTrace(stack, g_StackTraceSize, hash, stack_size);
			#else
				memset(stack, 0, g_StackTraceSize * sizeof(void*));
				CaptureStackBackTrace(0, g_StackTraceSize, stack, (PDWORD)&hash);
				for(stack_size = 0; stack_size<g_StackTraceSize; ++stack_size)
					if(!stack[stack_size])
						break;
			#endif
		#endif
	}

	//------------------------------------------------------------------------
	void Platform::SendPageState(
		bool send_memory,
		SendPageStateFunction send_page_state_function,
		void* p_context)
	{
		MEMORY_BASIC_INFORMATION info;
		memset(&info, 0, sizeof(info));

		uint64 addr = 0;

		HANDLE process = GetCurrentProcess();

		bool found_page = false;

		while(addr < MEMPRO_MAX_ADDRESS)
		{
			uint64 last_addr = addr;

			if(VirtualQueryEx(process, (void*)addr, &info, sizeof(info)) != 0)
			{
				if((info.State == MEM_RESERVE || info.State == MEM_COMMIT) && info.Protect != PAGE_NOACCESS)
				{
					PageState page_state;
					switch(info.State)
					{
						case MEM_RESERVE: page_state = MemPro::Reserved; break;
						case MEM_COMMIT: page_state = MemPro::Committed; break;
						default: page_state = MemPro::Committed; Platform::DebugBreak(); break;
					}

					PageType page_type;
					switch(info.Type)
					{
						case MEM_IMAGE: page_type = page_Image; break;
						case MEM_MAPPED: page_type = page_Mapped; break;
						case MEM_PRIVATE: page_type = page_Private; break;
						default: page_type = page_Unknown; break;
					}

					bool send_page_mem = send_memory && page_state == Committed && (info.Protect & (PAGE_NOACCESS | PAGE_EXECUTE | PAGE_GUARD)) == 0;

					send_page_state_function(
						info.BaseAddress,
						info.RegionSize,
						page_state,
						page_type,
						info.Protect,
						send_page_mem,
						MEMPRO_PAGE_SIZE,
						p_context);
				}

				addr += info.RegionSize;
				found_page = true;
			}
			else
			{
				if(!found_page)
					addr += MEMPRO_PAGE_SIZE;
				else
					break;		// VirtualQueryEx should only fail when it gets to the end, assuming it has found at least one page
			}

			if(addr < last_addr)		// handle wrap around
				break;
		}
	}
	
	//------------------------------------------------------------------------
	void Platform::GetVirtualMemStats(size_t& reserved, size_t& committed)
	{
		GenericPlatform::GetVirtualMemStats(reserved, committed);
	}

	//------------------------------------------------------------------------
	bool Platform::GetExtraModuleInfo(
		int64 ModuleBase,
		int& age,
		void* p_guid,
		int guid_size,
		char* p_pdb_filename,
		int pdb_filename_size)
	{
		return GenericPlatform::GetExtraModuleInfo(
			ModuleBase,
			age,
			p_guid,
			guid_size,
			p_pdb_filename,
			pdb_filename_size);
	}

	//------------------------------------------------------------------------
	void Platform::MemProEnumerateLoadedModules(
		EnumerateLoadedModulesCallbackFunction p_callback_function,
		void* p_context)
	{
		GenericPlatform::MemProEnumerateLoadedModules(p_callback_function, p_context);
	}

	//------------------------------------------------------------------------
	void Platform::DebugWrite(const char* p_message)
	{
		GenericPlatform::DebugWrite(p_message);
	}

	//------------------------------------------------------------------------
	void Platform::MemProMemoryBarrier()
	{
		#if (!defined(MIDL_PASS) && defined(_M_IX86) && !defined(_M_CEE_PURE)) || defined(MemoryBarrier)
			MemoryBarrier();
		#else
			std::atomic_thread_fence(std::memory_order_seq_cst);
		#endif
	}

	//------------------------------------------------------------------------
	EPlatform Platform::GetPlatform()
	{
		return Platform_Windows;
	}

	//------------------------------------------------------------------------
	int Platform::GetStackTraceSize()
	{
		return g_StackTraceSize;
	}

	//------------------------------------------------------------------------
	void Platform::MemCpy(void* p_dest, int dest_size, const void* p_source, int source_size)
	{
		GenericPlatform::MemCpy(p_dest, dest_size, p_source, source_size);
	}

	//------------------------------------------------------------------------
	void Platform::SPrintF(char* p_dest, int dest_size, const char* p_format, const char* p_str)
	{
		GenericPlatform::SPrintF(p_dest, dest_size, p_format, p_str);
	}

	//------------------------------------------------------------------------
	void Platform::MemProCreateFile(void* p_os_file_mem, int os_file_mem_size)
	{
		GenericPlatform::MemProCreateFile(p_os_file_mem, os_file_mem_size);
	}

	//------------------------------------------------------------------------
	void Platform::DestroyFile(void* p_os_file_mem)
	{
		GenericPlatform::DestroyFile(p_os_file_mem);
	}

	//------------------------------------------------------------------------
	bool Platform::OpenFileForWrite(void* p_os_file_mem, const char* p_filename)
	{
		return GenericPlatform::OpenFileForWrite(p_os_file_mem, p_filename);
	}

	//------------------------------------------------------------------------
	void Platform::CloseFile(void* p_os_file_mem)
	{
		GenericPlatform::CloseFile(p_os_file_mem);
	}

	//------------------------------------------------------------------------
	void Platform::FlushFile(void* p_os_file_mem)
	{
		GenericPlatform::FlushFile(p_os_file_mem);
	}

	//------------------------------------------------------------------------
	bool Platform::WriteFile(void* p_os_file_mem, const void* p_data, int size)
	{
		return GenericPlatform::WriteFile(p_os_file_mem, p_data, size);
	}

	//------------------------------------------------------------------------
	#ifdef MEMPRO_WRITE_DUMP
	void Platform::GetDumpFilename(char* p_filename, int max_length)
	{
		GenericPlatform::GetDumpFilename(p_filename, max_length);
	}
	#endif
}

//------------------------------------------------------------------------
#elif defined(MEMPRO_PLATFORM_XBOXONE)

	// implemented in MemProXboxOne.cpp - contact slynch@puredevsoftware.com for this platform

//------------------------------------------------------------------------
#elif defined(MEMPRO_PLATFORM_XBOX360)

	// implemented in MemProXbox360.cpp - contact slynch@puredevsoftware.com for this platform

//------------------------------------------------------------------------
#elif defined(MEMPRO_PLATFORM_PS4)

	// implemented in MemProPS4.cpp - contact slynch@puredevsoftware.com for this platform

//------------------------------------------------------------------------
#elif defined(MEMPRO_PLATFORM_PS5)

// implemented in MemProPS5.cpp - contact slynch@puredevsoftware.com for this platform

//------------------------------------------------------------------------
#elif defined(MEMPRO_PLATFORM_UNIX)

//------------------------------------------------------------------------
namespace MemPro
{
	//------------------------------------------------------------------------
	void Platform::CreateLock(void* p_os_lock_mem, int os_lock_mem_size)
	{
		GenericPlatform::CreateLock(p_os_lock_mem, os_lock_mem_size);
	}

	//------------------------------------------------------------------------
	void Platform::DestroyLock(void* p_os_lock_mem)
	{
		GenericPlatform::DestroyLock(p_os_lock_mem);
	}

	//------------------------------------------------------------------------
	void Platform::TakeLock(void* p_os_lock_mem)
	{
		GenericPlatform::TakeLock(p_os_lock_mem);
	}

	//------------------------------------------------------------------------
	void Platform::ReleaseLock(void* p_os_lock_mem)
	{
		GenericPlatform::ReleaseLock(p_os_lock_mem);
	}

	//------------------------------------------------------------------------
	#ifndef MEMPRO_WRITE_DUMP
	void Platform::UninitialiseSockets()
	{
		GenericPlatform::UninitialiseSockets();
	}
	#endif

	//------------------------------------------------------------------------
	#ifndef MEMPRO_WRITE_DUMP
	void Platform::CreateSocket(void* p_os_socket_mem, int os_socket_mem_size)
	{
		GenericPlatform::CreateSocket(p_os_socket_mem, os_socket_mem_size);
	}
	#endif

	//------------------------------------------------------------------------
	#ifndef MEMPRO_WRITE_DUMP
	bool Platform::IsValidSocket(const void* p_os_socket_mem)
	{
		return GenericPlatform::IsValidSocket(p_os_socket_mem);
	}
	#endif

	//------------------------------------------------------------------------
	#ifndef MEMPRO_WRITE_DUMP
	void Platform::Disconnect(void* p_os_socket_mem)
	{
		GenericPlatform::Disconnect(p_os_socket_mem);
	}
	#endif

	//------------------------------------------------------------------------
	#ifndef MEMPRO_WRITE_DUMP
	bool Platform::StartListening(void* p_os_socket_mem)
	{
		return GenericPlatform::StartListening(p_os_socket_mem);
	}
	#endif

	//------------------------------------------------------------------------
	#ifndef MEMPRO_WRITE_DUMP
	bool Platform::BindSocket(void* p_os_socket_mem, const char* p_port)
	{
		return GenericPlatform::BindSocket(p_os_socket_mem, p_port);
	}
	#endif

	//------------------------------------------------------------------------
	#ifndef MEMPRO_WRITE_DUMP
	bool Platform::AcceptSocket(void* p_os_socket_mem, void* p_client_os_socket_mem)
	{
		return GenericPlatform::AcceptSocket(p_os_socket_mem, p_client_os_socket_mem);
	}
	#endif

	//------------------------------------------------------------------------
	#ifndef MEMPRO_WRITE_DUMP
	bool Platform::SocketSend(void* p_os_socket_mem, void* p_buffer, int size)
	{
		return GenericPlatform::SocketSend(p_os_socket_mem, p_buffer, size);
	}
	#endif

	//------------------------------------------------------------------------
	#ifndef MEMPRO_WRITE_DUMP
	int Platform::SocketReceive(void* p_os_socket_mem, void* p_buffer, int size)
	{
		return GenericPlatform::SocketReceive(p_os_socket_mem, p_buffer, size);
	}
	#endif

	//------------------------------------------------------------------------
	void Platform::MemProCreateEvent(
		void* p_os_event_mem,
		int os_event_mem_size,
		bool initial_state,
		bool auto_reset)
	{
		GenericPlatform::MemProCreateEvent(
			p_os_event_mem,
			os_event_mem_size,
			initial_state,
			auto_reset);
	}

	//------------------------------------------------------------------------
	void Platform::DestroyEvent(void* p_os_event_mem)
	{
		GenericPlatform::DestroyEvent(p_os_event_mem);
	}

	//------------------------------------------------------------------------
	void Platform::SetEvent(void* p_os_event_mem)
	{
		GenericPlatform::SetEvent(p_os_event_mem);
	}

	//------------------------------------------------------------------------
	void Platform::ResetEvent(void* p_os_event_mem)
	{
		GenericPlatform::ResetEvent(p_os_event_mem);
	}

	//------------------------------------------------------------------------
	int Platform::WaitEvent(void* p_os_event_mem, int timeout)
	{
		return GenericPlatform::WaitEvent(p_os_event_mem, timeout);
	}

	//------------------------------------------------------------------------
	void Platform::CreateThread(void* p_os_thread_mem, int os_thread_mem_size)
	{
		GenericPlatform::CreateThread(p_os_thread_mem, os_thread_mem_size);
	}

	//------------------------------------------------------------------------
	void Platform::DestroyThread(void* p_os_thread_mem)
	{
		GenericPlatform::DestroyThread(p_os_thread_mem);
	}

	//------------------------------------------------------------------------
	int Platform::StartThread(void* p_os_thread_mem, ThreadMain p_thread_main, void* p_param)
	{
		return GenericPlatform::StartThread(p_os_thread_mem, p_thread_main, p_param);
	}

	//------------------------------------------------------------------------
	bool Platform::IsThreadAlive(const void* p_os_thread_mem)
	{
		return GenericPlatform::IsThreadAlive(p_os_thread_mem);
	}

	//------------------------------------------------------------------------
	int64 Platform::MemProInterlockedCompareExchange(int64 volatile *dest, int64 exchange, int64 comperand)
	{
		return GenericPlatform::MemProInterlockedCompareExchange(dest, exchange, comperand);
	}

	//------------------------------------------------------------------------
	int64 Platform::MemProInterlockedExchangeAdd(int64 volatile *Addend, int64 Value)
	{
		return GenericPlatform::MemProInterlockedExchangeAdd(Addend, Value);
	}

	//------------------------------------------------------------------------
	void Platform::SwapEndian(unsigned int& value)
	{
		GenericPlatform::SwapEndian(value);
	}

	//------------------------------------------------------------------------
	void Platform::SwapEndian(uint64& value)
	{
		GenericPlatform::SwapEndian(value);
	}

	//------------------------------------------------------------------------
	void Platform::DebugBreak()
	{
		GenericPlatform::DebugBreak();
	}

	//------------------------------------------------------------------------
	void* Platform::Alloc(int size)
	{
		return GenericPlatform::Alloc(size);
	}

	//------------------------------------------------------------------------
	void Platform::Free(void* p, int size)
	{
		GenericPlatform::Free(p, size);
	}

	//------------------------------------------------------------------------
	int64 Platform::GetHiResTimer()
	{
		timeval curr;
		gettimeofday(&curr, NULL);
		return ((int64)curr.tv_sec) * 1000000 + curr.tv_usec;
	}

	//------------------------------------------------------------------------
	// very innacurate, but portable. Doesn't have to be exact for our needs
	int64 Platform::GetHiResTimerFrequency()
	{
		static bool calculated_frequency = false;
		static int64 frequency = 1;
		if (!calculated_frequency)
		{
			calculated_frequency = true;
			Platform::Sleep(100);
			int64 start = GetHiResTimer();
			Platform::Sleep(1000);
			int64 end = GetHiResTimer();
			frequency = end - start;
		}

		return frequency;
	}

	//------------------------------------------------------------------------
	void Platform::SetThreadName(unsigned int thread_id, const char* p_name)
	{
		GenericPlatform::SetThreadName(thread_id, p_name);
	}

	//------------------------------------------------------------------------
	void Platform::Sleep(int ms)
	{
		GenericPlatform::Sleep(ms);
	}

	//------------------------------------------------------------------------
	void Platform::GetStackTrace(void** stack, int& stack_size, unsigned int& hash)
	{
		stack_size = backtrace(stack, g_StackTraceSize);
		hash = GetHashAndStackSize(stack, stack_size);
	}

	//------------------------------------------------------------------------
	void Platform::SendPageState(
		bool send_memory,
		SendPageStateFunction send_page_state_function,
		void* p_context)
	{
		GenericPlatform::SendPageState(
			send_memory,
			send_page_state_function,
			p_context);
	}

	//------------------------------------------------------------------------
	void Platform::GetVirtualMemStats(size_t& reserved, size_t& committed)
	{
		GenericPlatform::GetVirtualMemStats(reserved, committed);
	}

	//------------------------------------------------------------------------
	bool Platform::GetExtraModuleInfo(
		int64 ModuleBase,
		int& age,
		void* p_guid,
		int guid_size,
		char* p_pdb_filename,
		int pdb_filename_size)
	{
		return GenericPlatform::GetExtraModuleInfo(
			ModuleBase,
			age,
			p_guid,
			guid_size,
			p_pdb_filename,
			pdb_filename_size);
	}

	//------------------------------------------------------------------------
	void Platform::MemProEnumerateLoadedModules(
		EnumerateLoadedModulesCallbackFunction p_callback_function,
		void* p_context)
	{
		GenericPlatform::MemProEnumerateLoadedModules(p_callback_function, p_context);
	}

	//------------------------------------------------------------------------
	void Platform::DebugWrite(const char* p_message)
	{
		GenericPlatform::DebugWrite(p_message);
	}

	//------------------------------------------------------------------------
	void Platform::MemProMemoryBarrier()
	{
		__sync_synchronize();
	}

	//------------------------------------------------------------------------
	EPlatform Platform::GetPlatform()
	{
		return Platform_Unix;
	}

	//------------------------------------------------------------------------
	int Platform::GetStackTraceSize()
	{
		return g_StackTraceSize;
	}

	//------------------------------------------------------------------------
	void Platform::MemCpy(void* p_dest, int dest_size, const void* p_source, int source_size)
	{
		GenericPlatform::MemCpy(p_dest, dest_size, p_source, source_size);
	}

	//------------------------------------------------------------------------
	void Platform::SPrintF(char* p_dest, int dest_size, const char* p_format, const char* p_str)
	{
		GenericPlatform::SPrintF(p_dest, dest_size, p_format, p_str);
	}

	//------------------------------------------------------------------------
	void Platform::MemProCreateFile(void* p_os_file_mem, int os_file_mem_size)
	{
		GenericPlatform::MemProCreateFile(p_os_file_mem, os_file_mem_size);
	}

	//------------------------------------------------------------------------
	void Platform::DestroyFile(void* p_os_file_mem)
	{
		GenericPlatform::DestroyFile(p_os_file_mem);
	}

	//------------------------------------------------------------------------
	bool Platform::OpenFileForWrite(void* p_os_file_mem, const char* p_filename)
	{
		return GenericPlatform::OpenFileForWrite(p_os_file_mem, p_filename);
	}

	//------------------------------------------------------------------------
	void Platform::CloseFile(void* p_os_file_mem)
	{
		GenericPlatform::CloseFile(p_os_file_mem);
	}

	//------------------------------------------------------------------------
	void Platform::FlushFile(void* p_os_file_mem)
	{
		GenericPlatform::FlushFile(p_os_file_mem);
	}

	//------------------------------------------------------------------------
	bool Platform::WriteFile(void* p_os_file_mem, const void* p_data, int size)
	{
		return GenericPlatform::WriteFile(p_os_file_mem, p_data, size);
	}

	//------------------------------------------------------------------------
	#ifdef MEMPRO_WRITE_DUMP
	void Platform::GetDumpFilename(char* p_filename, int max_length)
	{
		GenericPlatform::GetDumpFilename(p_filename, max_length);
	}
	#endif
}

//------------------------------------------------------------------------
#elif defined(MEMPRO_PLATFORM_APPLE)

//------------------------------------------------------------------------
namespace MemPro
{
	//------------------------------------------------------------------------
	void Platform::CreateLock(void* p_os_lock_mem, int os_lock_mem_size)
	{
		GenericPlatform::CreateLock(p_os_lock_mem, os_lock_mem_size);
	}

	//------------------------------------------------------------------------
	void Platform::DestroyLock(void* p_os_lock_mem)
	{
		GenericPlatform::DestroyLock(p_os_lock_mem);
	}

	//------------------------------------------------------------------------
	void Platform::TakeLock(void* p_os_lock_mem)
	{
		GenericPlatform::TakeLock(p_os_lock_mem);
	}

	//------------------------------------------------------------------------
	void Platform::ReleaseLock(void* p_os_lock_mem)
	{
		GenericPlatform::ReleaseLock(p_os_lock_mem);
	}

	//------------------------------------------------------------------------
	#ifndef MEMPRO_WRITE_DUMP
	void Platform::UninitialiseSockets()
	{
		GenericPlatform::UninitialiseSockets();
	}
	#endif

	//------------------------------------------------------------------------
	#ifndef MEMPRO_WRITE_DUMP
	void Platform::CreateSocket(void* p_os_socket_mem, int os_socket_mem_size)
	{
		GenericPlatform::CreateSocket(p_os_socket_mem, os_socket_mem_size);
	}
	#endif

	//------------------------------------------------------------------------
	#ifndef MEMPRO_WRITE_DUMP
	bool Platform::IsValidSocket(const void* p_os_socket_mem)
	{
		return GenericPlatform::IsValidSocket(p_os_socket_mem);
	}
	#endif

	//------------------------------------------------------------------------
	#ifndef MEMPRO_WRITE_DUMP
	void Platform::Disconnect(void* p_os_socket_mem)
	{
		GenericPlatform::Disconnect(p_os_socket_mem);
	}
	#endif

	//------------------------------------------------------------------------
	#ifndef MEMPRO_WRITE_DUMP
	bool Platform::StartListening(void* p_os_socket_mem)
	{
		return GenericPlatform::StartListening(p_os_socket_mem);
	}
	#endif

	//------------------------------------------------------------------------
	#ifndef MEMPRO_WRITE_DUMP
	bool Platform::BindSocket(void* p_os_socket_mem, const char* p_port)
	{
		return GenericPlatform::BindSocket(p_os_socket_mem, p_port);
	}
	#endif

	//------------------------------------------------------------------------
	#ifndef MEMPRO_WRITE_DUMP
	bool Platform::AcceptSocket(void* p_os_socket_mem, void* p_client_os_socket_mem)
	{
		return GenericPlatform::AcceptSocket(p_os_socket_mem, p_client_os_socket_mem);
	}
	#endif

	//------------------------------------------------------------------------
	#ifndef MEMPRO_WRITE_DUMP
	bool Platform::SocketSend(void* p_os_socket_mem, void* p_buffer, int size)
	{
		return GenericPlatform::SocketSend(p_os_socket_mem, p_buffer, size);
	}
	#endif

	//------------------------------------------------------------------------
	#ifndef MEMPRO_WRITE_DUMP
	int Platform::SocketReceive(void* p_os_socket_mem, void* p_buffer, int size)
	{
		return GenericPlatform::SocketReceive(p_os_socket_mem, p_buffer, size);
	}
	#endif

	//------------------------------------------------------------------------
	void Platform::MemProCreateEvent(
		void* p_os_event_mem,
		int os_event_mem_size,
		bool initial_state,
		bool auto_reset)
	{
		GenericPlatform::MemProCreateEvent(
			p_os_event_mem,
			os_event_mem_size,
			initial_state,
			auto_reset);
	}

	//------------------------------------------------------------------------
	void Platform::DestroyEvent(void* p_os_event_mem)
	{
		GenericPlatform::DestroyEvent(p_os_event_mem);
	}

	//------------------------------------------------------------------------
	void Platform::SetEvent(void* p_os_event_mem)
	{
		GenericPlatform::SetEvent(p_os_event_mem);
	}

	//------------------------------------------------------------------------
	void Platform::ResetEvent(void* p_os_event_mem)
	{
		GenericPlatform::ResetEvent(p_os_event_mem);
	}

	//------------------------------------------------------------------------
	int Platform::WaitEvent(void* p_os_event_mem, int timeout)
	{
		return GenericPlatform::WaitEvent(p_os_event_mem, timeout);
	}

	//------------------------------------------------------------------------
	void Platform::CreateThread(void* p_os_thread_mem, int os_thread_mem_size)
	{
		GenericPlatform::CreateThread(p_os_thread_mem, os_thread_mem_size);
	}

	//------------------------------------------------------------------------
	void Platform::DestroyThread(void* p_os_thread_mem)
	{
		GenericPlatform::DestroyThread(p_os_thread_mem);
	}

	//------------------------------------------------------------------------
	int Platform::StartThread(void* p_os_thread_mem, ThreadMain p_thread_main, void* p_param)
	{
		return GenericPlatform::StartThread(p_os_thread_mem, p_thread_main, p_param);
	}

	//------------------------------------------------------------------------
	bool Platform::IsThreadAlive(const void* p_os_thread_mem)
	{
		return GenericPlatform::IsThreadAlive(p_os_thread_mem);
	}

	//------------------------------------------------------------------------
	int64 Platform::MemProInterlockedCompareExchange(int64 volatile *dest, int64 exchange, int64 comperand)
	{
		return GenericPlatform::MemProInterlockedCompareExchange(dest, exchange, comperand);
	}

	//------------------------------------------------------------------------
	int64 Platform::MemProInterlockedExchangeAdd(int64 volatile *Addend, int64 Value)
	{
		return GenericPlatform::MemProInterlockedExchangeAdd(Addend, Value);
	}

	//------------------------------------------------------------------------
	void Platform::SwapEndian(unsigned int& value)
	{
		GenericPlatform::SwapEndian(value);
	}

	//------------------------------------------------------------------------
	void Platform::SwapEndian(uint64& value)
	{
		GenericPlatform::SwapEndian(value);
	}

	//------------------------------------------------------------------------
	void Platform::DebugBreak()
	{
		GenericPlatform::DebugBreak();
	}

	//------------------------------------------------------------------------
	void* Platform::Alloc(int size)
	{
		return GenericPlatform::Alloc(size);
	}

	//------------------------------------------------------------------------
	void Platform::Free(void* p, int size)
	{
		GenericPlatform::Free(p, size);
	}

	//------------------------------------------------------------------------
	int64 Platform::GetHiResTimer()
	{
		timeval curr;
		gettimeofday(&curr, NULL);
		return ((int64)curr.tv_sec) * 1000000 + curr.tv_usec;
	}

	//------------------------------------------------------------------------
	// very innacurate, but portable. Doesn't have to be exact for our needs
	int64 Platform::GetHiResTimerFrequency()
	{
		static bool calculated_frequency = false;
		static int64 frequency = 1;
		if (!calculated_frequency)
		{
			calculated_frequency = true;
			Platform::Sleep(100);
			int64 start = GetHiResTimer();
			Platform::Sleep(1000);
			int64 end = GetHiResTimer();
			frequency = end - start;
		}

		return frequency;
	}

	//------------------------------------------------------------------------
	void Platform::SetThreadName(unsigned int thread_id, const char* p_name)
	{
		GenericPlatform::SetThreadName(thread_id, p_name);
	}

	//------------------------------------------------------------------------
	void Platform::Sleep(int ms)
	{
		GenericPlatform::Sleep(ms);
	}

	//------------------------------------------------------------------------
	void Platform::GetStackTrace(void** stack, int& stack_size, unsigned int& hash)
	{
		stack_size = backtrace(stack, g_StackTraceSize);
		hash = GetHashAndStackSize(stack, stack_size);
	}

	//------------------------------------------------------------------------
	void Platform::SendPageState(
		bool send_memory,
		SendPageStateFunction send_page_state_function,
		void* p_context)
	{
		GenericPlatform::SendPageState(
			send_memory,
			send_page_state_function,
			p_context);
	}

	//------------------------------------------------------------------------
	void Platform::GetVirtualMemStats(size_t& reserved, size_t& committed)
	{
		GenericPlatform::GetVirtualMemStats(reserved, committed);
	}

	//------------------------------------------------------------------------
	bool Platform::GetExtraModuleInfo(
		int64 ModuleBase,
		int& age,
		void* p_guid,
		int guid_size,
		char* p_pdb_filename,
		int pdb_filename_size)
	{
		return GenericPlatform::GetExtraModuleInfo(
			ModuleBase,
			age,
			p_guid,
			guid_size,
			p_pdb_filename,
			pdb_filename_size);
	}

	//------------------------------------------------------------------------
	void Platform::MemProEnumerateLoadedModules(
		EnumerateLoadedModulesCallbackFunction p_callback_function,
		void* p_context)
	{
		GenericPlatform::MemProEnumerateLoadedModules(p_callback_function, p_context);
	}

	//------------------------------------------------------------------------
	void Platform::DebugWrite(const char* p_message)
	{
		GenericPlatform::DebugWrite(p_message);
	}

	//------------------------------------------------------------------------
	void Platform::MemProMemoryBarrier()
	{
		__sync_synchronize();
	}

	//------------------------------------------------------------------------
	EPlatform Platform::GetPlatform()
	{
		return Platform_Unix;
	}

	//------------------------------------------------------------------------
	int Platform::GetStackTraceSize()
	{
		return g_StackTraceSize;
	}

	//------------------------------------------------------------------------
	void Platform::MemCpy(void* p_dest, int dest_size, const void* p_source, int source_size)
	{
		GenericPlatform::MemCpy(p_dest, dest_size, p_source, source_size);
	}

	//------------------------------------------------------------------------
	void Platform::SPrintF(char* p_dest, int dest_size, const char* p_format, const char* p_str)
	{
		GenericPlatform::SPrintF(p_dest, dest_size, p_format, p_str);
	}

	//------------------------------------------------------------------------
	void Platform::MemProCreateFile(void* p_os_file_mem, int os_file_mem_size)
	{
		GenericPlatform::MemProCreateFile(p_os_file_mem, os_file_mem_size);
	}

	//------------------------------------------------------------------------
	void Platform::DestroyFile(void* p_os_file_mem)
	{
		GenericPlatform::DestroyFile(p_os_file_mem);
	}

	//------------------------------------------------------------------------
	bool Platform::OpenFileForWrite(void* p_os_file_mem, const char* p_filename)
	{
		return GenericPlatform::OpenFileForWrite(p_os_file_mem, p_filename);
	}

	//------------------------------------------------------------------------
	void Platform::CloseFile(void* p_os_file_mem)
	{
		GenericPlatform::CloseFile(p_os_file_mem);
	}

	//------------------------------------------------------------------------
	void Platform::FlushFile(void* p_os_file_mem)
	{
		GenericPlatform::FlushFile(p_os_file_mem);
	}

	//------------------------------------------------------------------------
	bool Platform::WriteFile(void* p_os_file_mem, const void* p_data, int size)
	{
		return GenericPlatform::WriteFile(p_os_file_mem, p_data, size);
	}

	//------------------------------------------------------------------------
	#ifdef MEMPRO_WRITE_DUMP
	void Platform::GetDumpFilename(char* p_filename, int max_length)
	{
		GenericPlatform::GetDumpFilename(p_filename, max_length);
	}
	#endif
}

#endif

//------------------------------------------------------------------------
#endif		// #if MEMPRO_ENABLED

//------------------------------------------------------------------------
//
// Socket.cpp
//
#include <stdlib.h>
#include <new>

//------------------------------------------------------------------------
#if MEMPRO_ENABLED && !defined(MEMPRO_WRITE_DUMP)

//------------------------------------------------------------------------
void MemPro::Socket::Disconnect()
{
	Platform::Disconnect(m_OSSocketMem);
}

//------------------------------------------------------------------------
bool MemPro::Socket::StartListening()
{
	return Platform::StartListening(m_OSSocketMem);
}

//------------------------------------------------------------------------
bool MemPro::Socket::Bind(const char* p_port)
{
	return Platform::BindSocket(m_OSSocketMem, p_port);
}

//------------------------------------------------------------------------
bool MemPro::Socket::Accept(Socket& client_socket)
{
	return Platform::AcceptSocket(m_OSSocketMem, client_socket.m_OSSocketMem);
}

//------------------------------------------------------------------------
bool MemPro::Socket::Send(void* p_buffer, int size)
{
	return Platform::SocketSend(m_OSSocketMem, p_buffer, size);
}

//------------------------------------------------------------------------
int MemPro::Socket::Receive(void* p_buffer, int size)
{
	return Platform::SocketReceive(m_OSSocketMem, p_buffer, size);
}

//------------------------------------------------------------------------
#endif		// #if MEMPRO_ENABLED && !defined(MEMPRO_WRITE_DUMP)

//------------------------------------------------------------------------
//
// Thread.cpp
//
//------------------------------------------------------------------------
#if MEMPRO_ENABLED

//------------------------------------------------------------------------
MemPro::Thread::Thread()
{
	Platform::CreateThread(m_OSThread, sizeof(m_OSThread));
}

//------------------------------------------------------------------------
MemPro::Thread::~Thread()
{
	Platform::DestroyThread(m_OSThread);
}

//------------------------------------------------------------------------
int MemPro::Thread::CreateThread(ThreadMain p_thread_main, void* p_param)
{
	return Platform::StartThread(m_OSThread, p_thread_main, p_param);
}

//------------------------------------------------------------------------
#endif		// #if MEMPRO_ENABLED
