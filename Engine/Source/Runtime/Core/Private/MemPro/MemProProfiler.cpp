// Copyright Epic Games, Inc. All Rights Reserved.

#include "MemPro/MemProProfiler.h"

#if MEMPRO_ENABLED
#include "Misc/CString.h"
#include "Misc/CoreDelegates.h"
#include "Misc/Parse.h"
#include "HAL/IConsoleManager.h"

DEFINE_LOG_CATEGORY_STATIC(LogMemPro, Log, All);

/* NB. you can enable MemPro tracking on startup by adding something like this to the command line:
 *    -MemPro -MemProLLMTags="RHIMisc,EngineMisc"
 */

/* 
 * Main runtime switch for MemPro support
 */
int32 GMemProEnabled = 0;
static FAutoConsoleVariableRef CVarMemProEnable(
	TEXT("MemPro.Enabled"),
	GMemProEnabled,
	TEXT("Enable MemPro memory tracking.\n"),
	ECVF_Default
);


/* 
 * the LLM tag to track in MemPro, or ELLMTag::GenericTagCount to track all
 */
#if ENABLE_LOW_LEVEL_MEM_TRACKER
TStaticArray<bool,LLM_TAG_COUNT> MemProLLMTagsEnabled;
#endif //ENABLE_LOW_LEVEL_MEM_TRACKER



/* 
 * helper function to track a tag
 */
#if ENABLE_LOW_LEVEL_MEM_TRACKER
void FMemProProfiler::TrackTag( ELLMTag Tag )
{
	MemProLLMTagsEnabled[(int32)Tag] = true;
}

void ResetLLMTagArray(bool bValue)
{
	for (int i = 0; i < MemProLLMTagsEnabled.Num(); i++)
	{
		MemProLLMTagsEnabled[i] = bValue;
	}
}
#endif //ENABLE_LOW_LEVEL_MEM_TRACKER


/* 
 * helper function to track a tag given its name
 */
#if ENABLE_LOW_LEVEL_MEM_TRACKER
void FMemProProfiler::TrackTagsByName( const TCHAR* TagNamesStr )
{
	//sanity check
	if(TagNamesStr == nullptr || FCString::Strlen(TagNamesStr) == 0 )
	{
		UE_LOGF( LogMemPro, Display, "please specify an LLM tag or * to track all" );
		return;
	}


	if ( FCString::Stricmp(TagNamesStr, TEXT("none")) == 0 )
	{
		// Disable/reset tags
		ResetLLMTagArray(false);
	}
	else if ( FCString::Strcmp(TagNamesStr, TEXT("*") ) == 0 )
	{
		// Track all tags
		ResetLLMTagArray(true);
		UE_LOGF( LogMemPro, Display, "tracking all LLM tags" );
	}
	else
	{
		FString TagNamesString = TagNamesStr;
		// Strip leading/trailing quotes
		int32 Len = TagNamesString.Len();
		if (Len>1 && TagNamesString[Len-1] == '\"')
		{
			TagNamesString = TagNamesString.Mid(0,Len-1);
		}
		if (TagNamesString[0] == '\"')
		{
			TagNamesString = TagNamesString.Mid(1);
		}
		ResetLLMTagArray(false);
		TArray<FString> TagNames;
		const TCHAR* Delimiters[] = { TEXT(","), TEXT(" ") };
		if (TagNamesString.ParseIntoArray(TagNames, Delimiters, UE_ARRAY_COUNT(Delimiters), true) > 0)
		{
			for (const FString& TagName : TagNames)
			{
				// Find a specific tag to track
				uint64 TagIndex = (uint64)ELLMTag::Paused;
				if (FLowLevelMemTracker::Get().FindTagByName(*TagName, TagIndex) && TagIndex < LLM_TAG_COUNT)
				{
					TrackTag((ELLMTag)TagIndex);
					UE_LOGF(LogMemPro, Display, "tracking LLM tag \'%ls\'", *TagName);
				}
				else
				{
					UE_LOGF(LogMemPro, Display, "Unknown LLM tag \'%ls\'", *TagName);
				}
			}
		}
	}
}
#endif //ENABLE_LOW_LEVEL_MEM_TRACKER


/*
 * console command to get mempro to track a specific LLM tag
 */
#if ENABLE_LOW_LEVEL_MEM_TRACKER
static FAutoConsoleCommand MemProTrackLLMTag(
	TEXT("MemPro.LLMTag"),
	TEXT("Capture a specific LLM tag with MemPro"),
	FConsoleCommandWithArgsDelegate::CreateLambda( [](const TArray<FString>& Args )
	{
		FMemProProfiler::TrackTagsByName( (Args.Num() == 0) ? nullptr : *Args[0] );
	})
);
#endif //ENABLE_LOW_LEVEL_MEM_TRACKER


/*
 * query the port that MemPro might be using so other development tools can steer clear if necessary
 */
bool FMemProProfiler::IsUsingPort( uint32 Port )
{
#if defined(MEMPRO_WRITE_DUMP)
	return false;
#else
	return Port == FCStringAnsi::Atoi(MEMPRO_PORT);
#endif
}


/*
 * initialisation for MemPro.
 */
void FMemProProfiler::Init(const TCHAR* CmdLine)
{
#if ENABLE_LOW_LEVEL_MEM_TRACKER
	FString LLMTagsStr;
	if (FParse::Value(CmdLine, TEXT("MemProTags="), LLMTagsStr))
	{
		FMemProProfiler::TrackTagsByName(*LLMTagsStr);
	}
#endif
	if (FParse::Param(CmdLine, TEXT("MemPro")))
	{
		UE_LOGF(LogMemPro, Display, "MemPro enabled");
		GMemProEnabled = 1;
	}

	//shutdown MemPro when the engine is shutting down so that the send thread terminates cleanly
	FCoreDelegates::OnPreExit.AddLambda( []()
	{
		GMemProEnabled = 0;
		MemPro::Disconnect();
		//MemPro::Shutdown(); ...disabled for now as I was getting hangs on shutdown.
	});

}

#endif //MEMPRO_ENABLED
