// Copyright Epic Games, Inc. All Rights Reserved.

#include "Memory/CompositeBuffer.h"

#include "Algo/Accumulate.h"
#include "Algo/AllOf.h"
#include "HAL/PlatformString.h"
#include "Math/UnrealMathUtility.h"
#include "Memory/MemoryView.h"
#include "Misc/AssertionMacros.h"
#include "Templates/Function.h"
#include "Templates/RemoveReference.h"

const FCompositeBuffer FCompositeBuffer::Null;

FCompositeBuffer::~FCompositeBuffer() = default;

void FCompositeBuffer::Reset()
{
	Segments.Empty();
}

uint64 FCompositeBuffer::GetSize() const
{
	return Algo::TransformAccumulate(Segments, &FSharedBuffer::GetSize, uint64(0));
}

bool FCompositeBuffer::IsOwned() const
{
	return Algo::AllOf(Segments, &FSharedBuffer::IsOwned);
}

FCompositeBuffer FCompositeBuffer::MakeOwned() const &
{
	return FCompositeBuffer(*this).MakeOwned();
}

FCompositeBuffer FCompositeBuffer::MakeOwned() &&
{
	for (FSharedBuffer& Segment : Segments)
	{
		Segment = MoveTemp(Segment).MakeOwned();
	}
	return MoveTemp(*this);
}

FSharedBuffer FCompositeBuffer::ToShared() const &
{
	switch (Segments.Num())
	{
	case 0:
		return FSharedBuffer();
	case 1:
		return Segments[0];
	default:
		FUniqueBuffer Buffer = FUniqueBuffer::Alloc(GetSize());
		FMutableMemoryView BufferView = Buffer.GetView();
		for (const FSharedBuffer& SegmentBuffer : Segments)
		{
			BufferView = BufferView.CopyFrom(SegmentBuffer.GetView());
		}
		return Buffer.MoveToShared();
	}
}

FSharedBuffer FCompositeBuffer::ToShared() &&
{
	return Segments.Num() == 1 ? MoveTemp(Segments[0]) : AsConst(*this).ToShared();
}

FCompositeBuffer FCompositeBuffer::Mid(uint64 Offset, uint64 Size) const
{
	const uint64 BufferSize = GetSize();
	Offset = FMath::Min(Offset, BufferSize);
	Size = FMath::Min(Size, BufferSize - Offset);
	FCompositeBuffer Buffer;
	IterateRange(Offset, Size, [&Buffer](FMemoryView View, const FSharedBuffer& ViewOuter)
		{
			Buffer.Segments.Add(FSharedBuffer::MakeView(View, ViewOuter));
		});
	return Buffer;
}

FMemoryView FCompositeBuffer::ViewOrCopyRange(
	const uint64 Offset,
	const uint64 Size,
	FUniqueBuffer& CopyBuffer) const
{
	return ViewOrCopyRange(Offset, Size, CopyBuffer, FUniqueBuffer::Alloc);
}

FMemoryView FCompositeBuffer::ViewOrCopyRange(
	const uint64 Offset,
	const uint64 Size,
	FUniqueBuffer& CopyBuffer,
	TFunctionRef<FUniqueBuffer (uint64 Size)> Allocator) const
{
	FMemoryView View;
	IterateRange(Offset, Size, [Size, &View, &CopyBuffer, &Allocator, WriteView = FMutableMemoryView()](FMemoryView Segment) mutable
		{
			if (Size == Segment.GetSize())
			{
				View = Segment;
			}
			else
			{
				if (WriteView.IsEmpty())
				{
					if (CopyBuffer.GetSize() < Size)
					{
						CopyBuffer = Allocator(Size);
					}
					View = WriteView = CopyBuffer.GetView().Left(Size);
				}
				WriteView = WriteView.CopyFrom(Segment);
			}
		});
	return View;
}

void FCompositeBuffer::CopyTo(FMutableMemoryView Target, uint64 Offset) const
{
	IterateRange(Offset, Target.GetSize(), [Target](FMemoryView View, const FSharedBuffer& ViewOuter) mutable
		{
			Target = Target.CopyFrom(View);
		});
}

void FCompositeBuffer::IterateRange(uint64 Offset, uint64 Size, TFunctionRef<void (FMemoryView View)> Visitor) const
{
	IterateRange(Offset, Size, [Visitor](FMemoryView View, const FSharedBuffer& ViewOuter) { Visitor(View); });
}

void FCompositeBuffer::IterateRange(uint64 Offset, uint64 Size,
	TFunctionRef<void (FMemoryView View, const FSharedBuffer& ViewOuter)> Visitor) const
{
	checkf(Offset + Size <= GetSize(), TEXT("Failed to access %" UINT64_FMT " bytes at offset %" UINT64_FMT
		" of a composite buffer containing %" UINT64_FMT " bytes."), Size, Offset, GetSize());
	for (const FSharedBuffer& Segment : Segments)
	{
		if (const uint64 SegmentSize = Segment.GetSize(); Offset <= SegmentSize)
		{
			const FMemoryView View = Segment.GetView().Mid(Offset, Size);
			Offset = 0;
			if (Size == 0 || !View.IsEmpty())
			{
				Visitor(View, Segment);
			}
			Size -= View.GetSize();
			if (Size == 0)
			{
				break;
			}
		}
		else
		{
			Offset -= SegmentSize;
		}
	}
}

bool FCompositeBuffer::EqualBytes(const FCompositeBuffer& Other) const
{
	const uint64 Size = GetSize();
	if (Size != Other.GetSize())
	{
		return false;
	}
	uint64 Offset = 0;
	for (const FSharedBuffer& Segment : Segments)
	{
		bool bEqual = true;
		FMemoryView SegmentView = Segment.GetView();
		Other.IterateRange(Offset, Segment.GetSize(), [&bEqual, &SegmentView](FMemoryView OtherView)
		{
			bEqual = bEqual && SegmentView.Left(OtherView.GetSize()).EqualBytes(OtherView);
			SegmentView += OtherView.GetSize();
		});
		Offset += Segment.GetSize();
		if (!bEqual)
		{
			return false;
		}
	}
	return true;
}
