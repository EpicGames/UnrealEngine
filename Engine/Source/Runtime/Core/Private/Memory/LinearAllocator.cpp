// Copyright Epic Games, Inc. All Rights Reserved.

#include "Memory/LinearAllocator.h"

#include "BuildSettings.h"
#include "CoreGlobals.h"
#include "HAL/LowLevelMemTracker.h"
#include "Misc/AssertionMacros.h"
#include "Misc/ScopeExit.h"
#include "Misc/ScopeLock.h"
#include "Templates/AlignmentTemplates.h"

CORE_API FPersistentLinearAllocatorExtends GPersistentLinearAllocatorExtends;

namespace UE::LinearAllocator
{

static constexpr uint32 DefaultBlockSize = 64 * 1024;
static constexpr uint32 BlockAlignment = 8;
static constexpr uint32 DefaultAlignment = 8;

} // namespace UE::LinearAllocator


#if UE_ENABLE_LINEAR_VIRTUAL_ALLOCATOR

struct FPersistentLinearAllocator : public FLinearAllocator
{
	FPersistentLinearAllocator(SIZE_T ReserveMemorySize)
		: FLinearAllocator(ReserveMemorySize)
	{
		GPersistentLinearAllocatorExtends.Address = (uint64)VirtualMemory.GetVirtualPointer();
		GPersistentLinearAllocatorExtends.Size = (uint64)Reserved;
	}
};
 
FLinearAllocator::FLinearAllocator(SIZE_T ReserveMemorySize)
	: Reserved(ReserveMemorySize)
{
	if (FPlatformMemory::CanOverallocateVirtualMemory() && ReserveMemorySize)
	{
		VirtualMemory = VirtualMemory.AllocateVirtual(ReserveMemorySize);
		if (!VirtualMemory.GetVirtualPointer())
		{
			UE_LOG(LogMemory, Warning, TEXT("LinearAllocator failed to reserve %" SIZE_T_FMT " MB and will default to FMemory::Malloc instead"), ReserveMemorySize / 1024 / 1024);
			Reserved = 0;
		}
	}
	else
	{
#if PLATFORM_IOS || PLATFORM_TVOS
		UE_LOGF(LogMemory, Warning, "LinearAllocator requires com.apple.developer.kernel.extended-virtual-addressing entitlement to work");
#else
		UE_LOGF(LogMemory, Warning, "This platform does not allow to allocate more virtual memory than there is physical memory. LinearAllocator will default to FMemory::Malloc instead");
#endif
		Reserved = 0;
	}
}

void* FLinearAllocator::Allocate(SIZE_T Size, uint32 Alignment)
{
	Alignment = FMath::Max(Alignment, UE::LinearAllocator::DefaultAlignment);
	{
		void* Mem = nullptr;
		{
			FScopeLock AutoLock(&Lock);
			if (CanFit(Size, Alignment))
			{
				CurrentOffset = Align(CurrentOffset, Alignment);
				const SIZE_T NewOffset = CurrentOffset + Size;
				if (NewOffset > Committed)
				{
					LLM_PLATFORM_SCOPE(ELLMTag::FMalloc);
					const SIZE_T ToCommit = Align(NewOffset - Committed, FMath::Max((SIZE_T)VirtualMemory.GetCommitAlignment(),
						(SIZE_T)UE::LinearAllocator::DefaultBlockSize));
					VirtualMemory.Commit(Committed, ToCommit);
					Committed += ToCommit;
					LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelAlloc(ELLMTracker::Platform, (uint8*)VirtualMemory.GetVirtualPointer() + CurrentOffset, ToCommit));
				}
				Mem = (uint8*)VirtualMemory.GetVirtualPointer() + CurrentOffset;
				CurrentOffset += Size;
			}
		}
		if (Mem)
		{
			LLM(FLowLevelMemTracker::Get().OnLowLevelAlloc(ELLMTracker::Default, Mem, Size, ELLMTag::Untagged, ELLMAllocType::FMalloc));
			return Mem;
		}
	}

	static bool bOnce = false;
	if (!bOnce)
	{
		UE_LOG(LogMemory, Warning, TEXT("LinearAllocator exceeded %" SIZE_T_FMT " MB it reserved. Please tune PersistentAllocatorReserveSizeMB setting in [MemoryPools] ini group. Falling back to FMemory::Malloc"), Reserved / 1024 / 1024);
		bOnce = true;
	}
	return FMemory::Malloc(Size, Alignment);
}

bool FLinearAllocator::CanFit(SIZE_T Size, uint32 Alignment) const
{
	return Reserved >= Size + Align(CurrentOffset, Alignment);
}

bool FLinearAllocator::ContainsPointer(const void* Ptr) const
{
	return (uintptr_t)Ptr - (uintptr_t)VirtualMemory.GetVirtualPointer() < Reserved;
}

#else

FLinearAllocator::FLinearAllocator(SIZE_T /* ReserveMemorySize */)
	: UE::FLinearBlockAllocator(nullptr, 0U, ESPMode::ThreadSafe)
{
	// When !UE_ENABLE_LINEAR_VIRTUAL_ALLOCATOR , this allocator is a fallback from a higher level virtual allocator.
	// We do not need to support the initial reservation in that case. 
}

typedef FLinearAllocator FPersistentLinearAllocator;
#endif

namespace UE
{

FLinearBlockAllocator::FBlock* FLinearBlockAllocator::FBlock::GetHeaderFromBlockStart(void* Block, uint32 BlockSize)
{
	return reinterpret_cast<FBlock*>(reinterpret_cast<SIZE_T>(Block) + BlockSize - sizeof(FBlock));
}

void* FLinearBlockAllocator::FBlock::GetBlockStart()
{
	check(((SIZE_T)this) + sizeof(FBlock) >= (SIZE_T)BlockSize);
	return (void*)(((SIZE_T)this) + sizeof(FBlock) - (SIZE_T)BlockSize);
}

FLinearBlockAllocator::FLinearBlockAllocator(FMalloc* InInnerMalloc, uint32 InBlockSize,
	ESPMode InThreadMode)
	: InnerMalloc(InInnerMalloc)
	, DefaultBlockSize(InBlockSize != 0 ? InBlockSize : UE::LinearAllocator::DefaultBlockSize)
	, ThreadMode(InThreadMode)
{
	if (!InnerMalloc)
	{
		InnerMalloc = UE::Private::GMalloc;
	}
	static_assert(IsAligned(MinBlockSize, UE::LinearAllocator::BlockAlignment));
	DefaultBlockSize = FMath::Max(DefaultBlockSize, MinBlockSize);
	DefaultBlockSize = FMath::RoundUpToPowerOfTwo(DefaultBlockSize);
}

FLinearBlockAllocator::~FLinearBlockAllocator()
{
	checkf(ReferenceCount.load(std::memory_order_relaxed) == 0,
		TEXT("FLinearBlockAllocator is destroyed while still in use by a FLinearBlockAllocatorThreadAccessor."));

	LLM(const bool bDeclareAllocs = (InnerMalloc == UE::Private::GMalloc) && FLowLevelMemTracker::IsEnabled());

	while (FirstHeader)
	{
		FBlock* NextHeader = FirstHeader->Next;
		void* FirstBlock = FirstHeader->GetBlockStart();
		LLM(if (bDeclareAllocs)
		{
			FLowLevelMemTracker::Get().OnLowLevelFree(ELLMTracker::Default, FirstBlock, ELLMAllocType::FMalloc);
		});
		InnerMalloc->Free(FirstBlock);

		FirstHeader = NextHeader;
	}
}

void* FLinearBlockAllocator::Malloc(SIZE_T Size, uint32 Alignment)
{
	checkf(Size <= MAX_uint32 - sizeof(FBlock)*2,
		TEXT("FLinearBlockAllocator does not support allocations of size ~4GB or more. We store BlockSizes as uint32."));
	checkf(ReferenceCount.load(std::memory_order_relaxed) == 0,
		TEXT("FLinearBlockAllocator::Malloc is being used directly while in use by a FLinearBlockAllocatorThreadAccessor, this is not allowed, all malloc calls must go through a FLinearBlockAllocatorThreadAccessor."));
	Alignment = FMath::Max(Alignment, UE::LinearAllocator::DefaultAlignment);

	const bool bThreadSafe = IsThreadSafe();
	if (bThreadSafe)
	{
		Lock.Lock();
	}
	ON_SCOPE_EXIT
	{
		if (bThreadSafe)
		{
			Lock.Unlock();
		}
	};

	void* Result = TryAllocateFromBlock(Size, Alignment, LastHeader);
	if (Result)
	{
		return Result;
	}

	FBlock* NewHeader;
	if (!RequiresCustomAllocation(Size, Alignment))
	{
		NewHeader = AddDefaultBlock();
	}
	else
	{
		NewHeader = AddCustomBlock(Size, Alignment);
	}
	Result = TryAllocateFromBlock(Size, Alignment, NewHeader);
	check(Result);
	return Result;
}

SIZE_T FLinearBlockAllocator::GetAllocatedMemorySize() const
{
	FScopeLock ScopeLock(&Lock);
	return TotalAllocated;
}

bool FLinearBlockAllocator::IsThreadSafe() const
{
	return ThreadMode == ESPMode::ThreadSafe;
}

void* FLinearBlockAllocator::TryAllocateFromBlock(SIZE_T Size, uint32 Alignment, FBlock* Header)
{
	if (!Header)
	{
		return nullptr;
	}

	void* Block = Header->GetBlockStart();
	void* NextPtr = (void*)(((SIZE_T)Block) + ((SIZE_T)Header->NextOffset));
	void* AlignedPtr = Align(NextPtr, Alignment);
	SIZE_T NewOffset = ((SIZE_T)AlignedPtr - (SIZE_T)Block) + Size;
	if (NewOffset + sizeof(FBlock) > (SIZE_T)Header->BlockSize)
	{
		return nullptr;
	}

	Header->NextOffset = (uint32)NewOffset;
	return AlignedPtr;
}

bool FLinearBlockAllocator::RequiresCustomAllocation(SIZE_T Size, uint32 Alignment) const
{
	// Compute the worst case AlignmentPadding, which occurs if the Block pointer is aligned
	// only to BlockAlignment and no higher.
	SIZE_T StartingOffset = UE::LinearAllocator::BlockAlignment;
	SIZE_T Offset = StartingOffset;
	// Insert alignment padding
	Offset = Align(Offset, Alignment);
	// Add on the size.
	Offset += Size;
	// Add on the size of the Block header which is placed at the end of the allocated block.
	Offset += sizeof(FBlock);

	SIZE_T RequiredBlockSize = Offset - StartingOffset;
	return DefaultBlockSize < RequiredBlockSize;
}

FLinearBlockAllocator::FBlock* FLinearBlockAllocator::AddDefaultBlock()
{
	FBlock* NewHeader = AllocateBlock(DefaultBlockSize, UE::LinearAllocator::BlockAlignment);
	check(NewHeader);
	if (!FirstHeader)
	{
		check(!LastHeader);
		FirstHeader = NewHeader;
		LastHeader = NewHeader;
	}
	else
	{
		check(LastHeader);
		LastHeader->Next = NewHeader;
		LastHeader = NewHeader;
	}
	return NewHeader;
}

FLinearBlockAllocator::FBlock* FLinearBlockAllocator::AddCustomBlock(SIZE_T UserSize, uint32 UserAlignment)
{
	uint32 CustomAlignment = FMath::RoundUpToPowerOfTwo(FMath::Max(UserAlignment, UE::LinearAllocator::BlockAlignment));
	// User size was checked to fit within our smaller-type CustomBlockSize by the public Malloc functions
	// before calling AddCustomBlock.
	uint32 CustomBlockSize = (uint32)UserSize;
	CustomBlockSize = FMath::Max((CustomBlockSize + (uint32)sizeof(FBlock)), MinBlockSize);
	CustomBlockSize = Align(CustomBlockSize, UE::LinearAllocator::BlockAlignment);

	FBlock* NewHeader = AllocateBlock(CustomBlockSize, CustomAlignment);
	check(NewHeader);
	// Insert custom blocks at the beginning of the linked list, to avoid wasting the remaining memory in LastBlock.
	if (!FirstHeader)
	{
		check(!LastHeader);
		FirstHeader = NewHeader;
		LastHeader = NewHeader;
	}
	else
	{
		check(LastHeader);
		NewHeader->Next = FirstHeader;
		FirstHeader = NewHeader;
	}
	return NewHeader;
}

FLinearBlockAllocator::FBlock* FLinearBlockAllocator::AllocateBlock(uint32 BlockSize, uint32 BlockAlignment)
{
	check(BlockSize >= MinBlockSize);
	check(IsAligned(BlockSize, UE::LinearAllocator::BlockAlignment));
	check(IsAligned(BlockAlignment, UE::LinearAllocator::BlockAlignment));

	void* NewBlock = InnerMalloc->Malloc(BlockSize, BlockAlignment);
	check(NewBlock != nullptr);
	LLM(const bool bDeclareAllocs = (InnerMalloc == UE::Private::GMalloc) && FLowLevelMemTracker::IsEnabled());
	LLM(if (bDeclareAllocs)
	{
		FLowLevelMemTracker::Get().OnLowLevelAlloc(ELLMTracker::Default, NewBlock, BlockSize,
			ELLMTag::Untagged, ELLMAllocType::FMalloc);
	});
	TotalAllocated += BlockSize;

	// Note that NewHeader is aligned to at least UE::LinearAllocator::BlockAlignment, because BlockAllocation,
	// BlockSize, and sizeof(FBlock) are all aligned to at least UE::LinearAllocator::BlockAlignment.
	FBlock* NewHeader = FBlock::GetHeaderFromBlockStart(NewBlock, BlockSize);
	NewHeader->Next = nullptr;
	NewHeader->BlockSize = BlockSize;
	NewHeader->NextOffset = 0;
	return NewHeader;
}


FLinearBlockAllocatorThreadAccessor::FLinearBlockAllocatorThreadAccessor(FLinearBlockAllocator& InAllocator)
	: Allocator(&InAllocator)
{
	Allocator->ReferenceCount.fetch_add(1, std::memory_order_relaxed);
}

FLinearBlockAllocatorThreadAccessor::~FLinearBlockAllocatorThreadAccessor()
{
	if (Allocator)
	{
		Allocator->ReferenceCount.fetch_sub(1, std::memory_order_relaxed);
	}
}

FLinearBlockAllocatorThreadAccessor::FLinearBlockAllocatorThreadAccessor(
	FLinearBlockAllocatorThreadAccessor&& Other)
{
	Allocator = Other.Allocator;
	LastHeader = Other.LastHeader;
	Other.Allocator = nullptr;
	Other.LastHeader = nullptr;
}

void* FLinearBlockAllocatorThreadAccessor::Malloc(SIZE_T Size, uint32 Alignment)
{
	checkf(Size <= MAX_uint32 - sizeof(FLinearBlockAllocator::FBlock) * 2,
		TEXT("FLinearBlockAllocator does not support allocations of size ~4GB or more. We store BlockSizes as uint32."));
	// Malloc must not be called on cleared thread accessors (e.g. cleared by move constructor into another accessor)
	check(Allocator);

	Alignment = FMath::Max(Alignment, UE::LinearAllocator::DefaultAlignment);
	void* Result = Allocator->TryAllocateFromBlock(Size, Alignment, LastHeader);
	if (Result)
	{
		return Result;
	}

	FLinearBlockAllocator::FBlock* NewHeader;
	{
		FScopeLock FScopeLock(&Allocator->Lock);
		if (!Allocator->RequiresCustomAllocation(Size, Alignment))
		{
			NewHeader = Allocator->AddDefaultBlock();
			LastHeader = NewHeader;
		}
		else
		{
			NewHeader = Allocator->AddCustomBlock(Size, Alignment);
			// Do not modify LastHeader, keep allocating from the old one
		}
	}
	Result = Allocator->TryAllocateFromBlock(Size, Alignment, NewHeader);
	check(Result);
	return Result;
}

} // namespace UE

FLinearAllocator& GetPersistentLinearAllocator()
{
	// We have to make sure that PersistentLinearAllocator always reserves the amount of memory that's not multiple of 2 MB as it causes issues on platforms with transparent large pages
	static FPersistentLinearAllocator GPersistentLinearAllocator(BuildSettings::GetPersistentAllocatorReserveSize() + 64 * 1024);
	return GPersistentLinearAllocator;
}