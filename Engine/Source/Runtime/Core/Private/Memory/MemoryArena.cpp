// Copyright Epic Games, Inc. All Rights Reserved.

#include "Memory/MemoryArena.h"
#include "HAL/UnrealMemory.h"
#include "HAL/MallocAnsi.h"
#include "HAL/CriticalSection.h"
#include "Misc/ScopeRWLock.h"
#include <atomic>

PRAGMA_DISABLE_UNSAFE_TYPECAST_WARNINGS

#if UE_WITH_HEAPARENA
// dlmalloc for use in FHeapArena

extern "C" {
#define NO_MALLINFO 1
#define ONLY_MSPACES 1
#define USE_LOCKS 1
#define USE_SPIN_LOCKS 0
//#define USE_RECURSIVE_LOCKS 1

#include "ThirdParty/dlmalloc/malloc-2.8.6.h"

#if PLATFORM_WINDOWS
#	pragma warning(push)
#		pragma warning(disable : 4668)	// macro redefinition
#		include "Windows/AllowWindowsPlatformTypes.h"
#endif


#include "ThirdParty/dlmalloc/malloc-2.8.6.c"

#if PLATFORM_WINDOWS
#		include "Windows/HideWindowsPlatformTypes.h"
#	pragma warning(pop)
#endif
}
#endif

//////////////////////////////////////////////////////////////////////////
//
// Memory Arena tracking
//

static const int MaxArenaCount = 256;

union FArenaOrFreeLink
{
	FMemoryArena*	Arena;
	uint64			NextFreeArenaIndex;
};

FArenaOrFreeLink* GKnownArenas;

struct alignas(PLATFORM_CACHE_LINE_SIZE) FArenaAllocState
{
	FArenaOrFreeLink	KnownArenas[MaxArenaCount];

	FRWLock			Lock;
	uint64			NextFreeArenaIndex = 1;

	FArenaAllocState()
	{
		for (int i = 1; i < (MaxArenaCount - 1); ++i)
		{
			KnownArenas[i].NextFreeArenaIndex = i + 1;
		}

		KnownArenas[MaxArenaCount - 1].NextFreeArenaIndex = 0;

		GKnownArenas = KnownArenas;
	}

	~FArenaAllocState()
	{
	}
};

TAtomic<FArenaAllocState*> GTracker;

FArenaAllocState& ArenaTracker()
{
	// Try to avoid overhead from atomics due to the thread-safe
	// static initialization below.

	if (GTracker)
	{
		return *GTracker;
	}

	static FArenaAllocState Tracker;

	// TODO: Should have a FMallocArena as arena 0?

	GTracker.Store(&Tracker);

	return Tracker;
}

static uint16 AllocArenaId(FMemoryArena* Arena)
{
	FArenaAllocState& Tracker = ArenaTracker();

	FWriteScopeLock _(Tracker.Lock);

	const uint64 NewArenaId					= Tracker.NextFreeArenaIndex;

	check(NewArenaId < MaxArenaCount);

	const uint64 NextFree					= Tracker.KnownArenas[NewArenaId].NextFreeArenaIndex;
	Tracker.KnownArenas[NewArenaId].Arena	= Arena;
	Tracker.NextFreeArenaIndex				= NextFree;

	return (uint16)NewArenaId;
}

static void FreeArenaId(uint16 ArenaId, FMemoryArena* Arena)
{
	FArenaAllocState& Tracker = ArenaTracker();

	FWriteScopeLock _(Tracker.Lock);

	check(Tracker.KnownArenas[ArenaId].Arena == Arena);

	Tracker.KnownArenas[ArenaId].NextFreeArenaIndex = Tracker.NextFreeArenaIndex;
	Tracker.NextFreeArenaIndex						= ArenaId;
}

//////////////////////////////////////////////////////////////////////////

FMemoryArena& FArenaPointer::Arena() const
{
	const uint16 Index = GetArenaIndex();

	return *GKnownArenas[Index].Arena;
}

//////////////////////////////////////////////////////////////////////////

FMemoryArena::FMemoryArena()
{
	ArenaId = AllocArenaId(this);
}

FMemoryArena::~FMemoryArena()
{
	FreeArenaId(ArenaId, this);
}


UE_RESTRICT UE_NOALIAS void* FMemoryArena::Alloc(SIZE_T Size, SIZE_T Alignment)
{
	return InternalAlloc(Size, Alignment);
}

UE_NOALIAS void FMemoryArena::Free(const void* MemoryBlock)
{
	if (MemoryBlock == nullptr || ArenaFlags & FlagNoFree)
	{
		return;
	}

	return InternalFree(MemoryBlock, 0);
}

SIZE_T FMemoryArena::GetBlockSize(const void* MemoryBlock) const
{
	return InternalGetBlockSize(MemoryBlock);
}

const TCHAR* FMemoryArena::GetDebugName() const
{
	return InternalGetDebugName();
}

void FMemoryArena::InternalFree(const void* MemoryBlock, SIZE_T MemoryBlockSize)
{
}

const TCHAR* FMemoryArena::InternalGetDebugName() const
{
	return TEXT("(unnamed)");
}

//////////////////////////////////////////////////////////////////////////

FArenaPointer ArenaRealloc(FMemoryArena* Arena, void* InPtr, SIZE_T OldSize, SIZE_T NewSize, SIZE_T Alignment)
{
	if (!Arena)
	{
		return FArenaPointer(FMemory::Realloc(InPtr, NewSize, Alignment), 0);
	}

	if (!NewSize)
	{
		Arena->Free(InPtr);

		return FArenaPointer();
	}

	void* NewPtr = Arena->Alloc(NewSize, Alignment);

	if (InPtr)
	{
		FMemory::Memcpy(NewPtr, InPtr, FMath::Min(OldSize, NewSize));

		Arena->Free(InPtr);
	}

	return FArenaPointer(NewPtr, Arena->ArenaId);
}

FArenaPointer ArenaRealloc(FArenaPointer InPtr, SIZE_T OldSize, SIZE_T NewSize, SIZE_T Alignment)
{
	return ArenaRealloc(&InPtr.Arena(), InPtr.GetPointer(), OldSize, NewSize, Alignment);
}

//////////////////////////////////////////////////////////////////////////

#if UE_WITH_HEAPARENA

FHeapArena::FHeapArena()
{
	HeapHandle = create_mspace(1024 * 1024, /* locked */ 1);
}

FHeapArena::~FHeapArena()
{
	destroy_mspace(HeapHandle);
}

void* FHeapArena::InternalAlloc(SIZE_T Size, SIZE_T Alignment)
{
	return mspace_memalign(HeapHandle, Alignment, Size);
}

void FHeapArena::InternalFree(const void* MemoryBlock, SIZE_T MemoryBlockSize)
{
	return mspace_free(HeapHandle, (void*) MemoryBlock);
}

SIZE_T FHeapArena::InternalGetBlockSize(const void* MemoryBlock) const
{
	return mspace_usable_size(MemoryBlock);
}

const TCHAR* FHeapArena::InternalGetDebugName() const
{
	return TEXT("HeapArena");
}

#endif

//////////////////////////////////////////////////////////////////////////

FMallocArena::FMallocArena()
{
}

FMallocArena::~FMallocArena()
{
}

void* FMallocArena::InternalAlloc(SIZE_T Size, SIZE_T Alignment)
{
	return FMemory::Malloc(Size, Alignment);
}

void FMallocArena::InternalFree(const void* MemoryBlock, SIZE_T MemoryBlockSize)
{
	FMemory::Free(const_cast<void*>(MemoryBlock));
}

SIZE_T FMallocArena::InternalGetBlockSize(const void* MemoryBlock) const
{
	return FMemory::GetAllocSize(const_cast<void*>(MemoryBlock));
}

const TCHAR* FMallocArena::InternalGetDebugName() const
{
	return TEXT("MallocArena");
}

//////////////////////////////////////////////////////////////////////////

FMallocAnsi GAnsiMalloc;

FAnsiArena::FAnsiArena() = default;
FAnsiArena::~FAnsiArena() = default;

void* FAnsiArena::InternalAlloc(SIZE_T Size, SIZE_T Alignment)
{
	return GAnsiMalloc.Malloc(Size, Alignment);
}

void FAnsiArena::InternalFree(const void* MemoryBlock, SIZE_T MemoryBlockSize)
{
	return GAnsiMalloc.Free(const_cast<void*>(MemoryBlock));
}

SIZE_T FAnsiArena::InternalGetBlockSize(const void* MemoryBlock) const
{
	SIZE_T Size = 0;
	GAnsiMalloc.GetAllocationSize(const_cast<void*>(MemoryBlock), /* out */ Size);
	return Size;
}

const TCHAR* FAnsiArena::InternalGetDebugName() const
{
	return TEXT("AnsiArena");
}


#if UE_WITH_ARENAMAP

//////////////////////////////////////////////////////////////////////////
//
// Arena map - arenas need to register with the arena map in order to
//             support pointer -> arena mapping.
//

// TODO: These need adjusting for different targets
//
// We could probably also be a bit more optimistic here in some cases and
// not permit the full address range for memory arenas, in order to save
// some memory.

const unsigned		PageBits		= 16;
const unsigned int	PageAlignment	= 1u << PageBits;

const unsigned int	PointerBits		= 48;
const unsigned int	SubrangeBits	= 30;

const unsigned int	PagesInSubrange = 1u << (SubrangeBits - PageBits);
const unsigned int	SubrangeCount	= 1u << (PointerBits - SubrangeBits);

typedef FMemoryArena*	SubrangeArray[PagesInSubrange];
SubrangeArray*			GSubrangeArrays[SubrangeCount];

void FArenaMap::Initialize()
{
}

void FArenaMap::Reset()
{
	for (int i = 0; i < SubrangeCount; ++i)
	{
		if (SubrangeArray* Subrange = GSubrangeArrays[i])
		{
			FPlatformMemory::FPlatformVirtualMemoryBlock Block(Subrange, sizeof(SubrangeArray));

			Block.FreeVirtual();

			GSubrangeArrays[i] = nullptr;
		}
	}
}

void FArenaMap::SetRangeToArena(const void* VaBase, SIZE_T VaSize, FMemoryArena* ArenaPtr)
{
	if (VaSize < PageAlignment)
	{
		// Cannot resolve blocks below arena map resolution
		PLATFORM_BREAK();
	}

	if (VaSize & (PageAlignment - 1))
	{
		// VA range size must be an multiple of arena map resolution
		PLATFORM_BREAK();
	}

	UPTRINT			VaCursor = UPTRINT(VaBase);
	const UPTRINT	VaEnd = VaCursor + VaSize;

	if (VaCursor & (PageAlignment - 1))
	{
		// VA range must start on a arena map boundary
		PLATFORM_BREAK();
	}

	const uint32 SubrangeStartIndex = (VaCursor >> SubrangeBits);
	const uint32 SubrangeEndIndex	= ((VaEnd - 1) >> SubrangeBits);

	if ((SubrangeStartIndex >= SubrangeCount) || (SubrangeEndIndex >= SubrangeCount))
	{
		// Out of bounds
		PLATFORM_BREAK();
	}

	// We use std atomics here just because we don't have acquire semantics on load in TAtomic
	auto SubrangeArrays = reinterpret_cast<std::atomic<SubrangeArray*>*>(&GSubrangeArrays[0]);

	// Iterate over all subranges, associating all covered page entries with arena
	
	for (uint32 CurrentSubrange = SubrangeStartIndex; CurrentSubrange <= SubrangeEndIndex; ++CurrentSubrange)
	{
		SubrangeArray* Subrange = SubrangeArrays[CurrentSubrange].load(std::memory_order_acquire);

		if (!Subrange)
		{
			// No range set - initialize speculatively. If another thread gets there first we'll let this block of memory go

			FPlatformMemory::FPlatformVirtualMemoryBlock Block = FPlatformMemory::FPlatformVirtualMemoryBlock::AllocateVirtual(sizeof(SubrangeArray));
			Block.Commit();

			// TODO: Do all platforms AllocateVirtual return zero initialized memory?

			Subrange = reinterpret_cast<SubrangeArray*>(Block.GetVirtualPointer());

			SubrangeArray* NullRange = nullptr;

			if (!SubrangeArrays[CurrentSubrange].compare_exchange_strong(NullRange, Subrange, std::memory_order_acq_rel))
			{
				// CAS failed - the subrange is now set
				Block.FreeVirtual();

				Subrange = SubrangeArrays[CurrentSubrange].load(std::memory_order_acquire);
			}
		}

		// Compute bounds covered by subrange

		const UPTRINT	VaSubrangeEnd	= (CurrentSubrange + 1) * (1 << SubrangeBits);				// End of current subrange
		const UPTRINT	VaSegmentEnd	= FMath::Min(VaEnd, VaSubrangeEnd);							// Clip set range to current subrange

		const SIZE_T	StartIndex		= (VaCursor >> PageBits) & (PagesInSubrange - 1);			// Index modulo subrange for current VA pointer
		const SIZE_T	EndIndex		= ((VaSegmentEnd - 1) >> PageBits) & (PagesInSubrange - 1);	// Index modulo subrange for VA (segment end) *inclusive*

		for (SIZE_T i = StartIndex; i <= EndIndex; ++i)
		{
			(*Subrange)[i] = ArenaPtr;
		}

		VaCursor = VaSegmentEnd;
	}
}

void FArenaMap::ClearRange(const void* VaBase, SIZE_T VaSize)
{
	SetRangeToArena(VaBase, VaSize, nullptr);
}

FMemoryArena* FArenaMap::MapPtrToArena(const void* VaBase)
{
	const UPTRINT	VaPtr			= UPTRINT(VaBase);

	const SIZE_T	SubrangeIndex	= (VaPtr >> SubrangeBits);

	if (SubrangeIndex >= SubrangeCount)
	{
		PLATFORM_BREAK();
	}

	// We use std atomics here just because we don't have acquire semantics on load in TAtomic
	auto SubrangeArrays = reinterpret_cast<std::atomic<SubrangeArray*>*>(&GSubrangeArrays[0]);

	SubrangeArray* Subrange = SubrangeArrays[SubrangeIndex].load(std::memory_order_acquire);

	if (Subrange)
	{
		const SIZE_T	PageIndex = (VaPtr >> PageBits) & (PageAlignment - 1);

		FMemoryArena* PageArena = (*Subrange)[PageIndex];

		return PageArena;
	}

	return nullptr;
}

#endif

//////////////////////////////////////////////////////////////////////////

PRAGMA_RESTORE_UNSAFE_TYPECAST_WARNINGS
