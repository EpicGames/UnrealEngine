// Copyright Epic Games, Inc. All Rights Reserved.

#include "Memory/SharedBuffer.h"

#include "HAL/UnrealMemory.h"
#include "Templates/RemoveReference.h"

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

namespace UE::SharedBuffer::Private
{

template <typename FOps>
inline TBufferOwnerPtr<FOps>::TBufferOwnerPtr(FBufferOwner* const InOwner)
	: Owner(InOwner)
{
	if (InOwner)
	{
		checkf(!FOps::HasRef(*InOwner), TEXT("FBufferOwner is referenced by another TBufferOwnerPtr. ")
			TEXT("Construct this from an existing pointer instead of a raw pointer."));
		FOps::AddRef(*InOwner);
	}
}

template <typename FOps>
inline void TBufferOwnerPtr<FOps>::Reset()
{
	FOps::Release(Owner);
	Owner = nullptr;
}

class FBufferOwnerHeap final : public FBufferOwner
{
public:
	static_assert(~uint64(0) <= ~SIZE_T(0));

	inline explicit FBufferOwnerHeap(uint64 Size)
		: FBufferOwner(FMemory::Malloc(Size), Size)
	{
		SetIsMaterialized();
		SetIsOwned();
	}

protected:
	virtual void FreeBuffer() final
	{
		FMemory::Free(GetData());
	}
};

class FBufferOwnerView final : public FBufferOwner
{
public:
	inline FBufferOwnerView(void* Data, uint64 Size)
		: FBufferOwner(Data, Size)
	{
		SetIsMaterialized();
	}

	inline FBufferOwnerView(const void* Data, uint64 Size)
		: FBufferOwnerView(const_cast<void*>(Data), Size)
	{
		SetIsImmutable();
	}

protected:
	virtual void FreeBuffer() final
	{
	}
};

class FBufferOwnerOuterView final : public FBufferOwner
{
public:
	template <typename OuterBufferType, decltype(FSharedBuffer(DeclVal<OuterBufferType>()))* = nullptr>
	inline FBufferOwnerOuterView(void* Data, uint64 Size, OuterBufferType&& InOuterBuffer)
		: FBufferOwner(Data, Size)
		, OuterBuffer(Forward<OuterBufferType>(InOuterBuffer))
	{
		check(OuterBuffer.GetView().Contains(MakeMemoryView(GetData(), GetSize())));
		SetIsMaterialized();
		SetIsImmutable();
		if (OuterBuffer.IsOwned())
		{
			SetIsOwned();
		}
	}

protected:
	virtual void FreeBuffer() final
	{
		OuterBuffer.Reset();
	}

private:
	FSharedBuffer OuterBuffer;
};

} // UE::SharedBuffer::Private

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FUniqueBuffer FUniqueBuffer::Alloc(uint64 InSize)
{
	return FUniqueBuffer(new UE::SharedBuffer::Private::FBufferOwnerHeap(InSize));
}

FUniqueBuffer FUniqueBuffer::AllocZeroed(uint64 InSize)
{
	FUniqueBuffer Buffer = Alloc(InSize);
	FMemory::Memzero(Buffer.GetData(), Buffer.GetSize());
	return Buffer;
}

FUniqueBuffer FUniqueBuffer::Clone(FMemoryView View)
{
	FUniqueBuffer Buffer = Alloc(View.GetSize());
	Buffer.GetView().CopyFrom(View);
	return Buffer;
}

FUniqueBuffer FUniqueBuffer::Clone(const void* Data, uint64 Size)
{
	return Clone(MakeMemoryView(Data, Size));
}

FUniqueBuffer FUniqueBuffer::MakeView(FMutableMemoryView View)
{
	return MakeView(View.GetData(), View.GetSize());
}

FUniqueBuffer FUniqueBuffer::MakeView(void* Data, uint64 Size)
{
	return FUniqueBuffer(new UE::SharedBuffer::Private::FBufferOwnerView(Data, Size));
}

FUniqueBuffer::FUniqueBuffer(FBufferOwner* InOwner)
	: Owner(InOwner)
{
}

FUniqueBuffer::FUniqueBuffer(FOwnerPtrType&& SharedOwner)
	: Owner(MoveTemp(SharedOwner))
{
}

void FUniqueBuffer::Reset()
{
	Owner.Reset();
}

FUniqueBuffer FUniqueBuffer::MakeOwned() &&
{
	return IsOwned() ? MoveTemp(*this) : Clone(GetView());
}

void FUniqueBuffer::Materialize() const
{
	if (Owner)
	{
		Owner->Materialize();
	}
}

FSharedBuffer FUniqueBuffer::MoveToShared()
{
	return FSharedBuffer(ToPrivateOwnerPtr(MoveTemp(*this)));
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

template <typename OuterBufferType, decltype(FSharedBuffer(DeclVal<OuterBufferType>()))* = nullptr>
static FSharedBuffer MakeSharedBufferViewWithOuter(FMemoryView View, OuterBufferType&& OuterBuffer)
{
	if (OuterBuffer.IsNull())
	{
		return FSharedBuffer::MakeView(View);
	}
	if (View == OuterBuffer.GetView())
	{
		return Forward<OuterBufferType>(OuterBuffer);
	}
	return FSharedBuffer(new UE::SharedBuffer::Private::FBufferOwnerOuterView(
		const_cast<void*>(View.GetData()), View.GetSize(), Forward<OuterBufferType>(OuterBuffer)));
}

FSharedBuffer FSharedBuffer::Clone(FMemoryView View)
{
	return FUniqueBuffer::Clone(View).MoveToShared();
}

FSharedBuffer FSharedBuffer::Clone(const void* Data, uint64 Size)
{
	return FUniqueBuffer::Clone(Data, Size).MoveToShared();
}

FSharedBuffer FSharedBuffer::MakeView(FMemoryView View)
{
	return MakeView(View.GetData(), View.GetSize());
}

FSharedBuffer FSharedBuffer::MakeView(FMemoryView View, FSharedBuffer&& OuterBuffer)
{
	return MakeSharedBufferViewWithOuter(View, MoveTemp(OuterBuffer));
}

FSharedBuffer FSharedBuffer::MakeView(FMemoryView View, const FSharedBuffer& OuterBuffer)
{
	return MakeSharedBufferViewWithOuter(View, OuterBuffer);
}

FSharedBuffer FSharedBuffer::MakeView(const void* Data, uint64 Size)
{
	return FSharedBuffer(new UE::SharedBuffer::Private::FBufferOwnerView(Data, Size));
}

FSharedBuffer FSharedBuffer::MakeView(const void* Data, uint64 Size, FSharedBuffer&& OuterBuffer)
{
	return MakeSharedBufferViewWithOuter(MakeMemoryView(Data, Size), MoveTemp(OuterBuffer));
}

FSharedBuffer FSharedBuffer::MakeView(const void* Data, uint64 Size, const FSharedBuffer& OuterBuffer)
{
	return MakeSharedBufferViewWithOuter(MakeMemoryView(Data, Size), OuterBuffer);
}

FSharedBuffer::FSharedBuffer(FBufferOwner* InOwner)
	: Owner(InOwner)
{
}

FSharedBuffer::FSharedBuffer(FOwnerPtrType&& SharedOwner)
	: Owner(MoveTemp(SharedOwner))
{
}

FSharedBuffer::FSharedBuffer(const FWeakOwnerPtrType& WeakOwner)
	: Owner(WeakOwner)
{
}

void FSharedBuffer::Reset()
{
	Owner.Reset();
}

FSharedBuffer FSharedBuffer::MakeOwned() const &
{
	return IsOwned() ? *this : Clone(GetView());
}

FSharedBuffer FSharedBuffer::MakeOwned() &&
{
	return IsOwned() ? MoveTemp(*this) : Clone(GetView());
}

void FSharedBuffer::Materialize() const
{
	if (Owner)
	{
		Owner->Materialize();
	}
}

FUniqueBuffer FSharedBuffer::MoveToUnique()
{
	FOwnerPtrType ExistingOwner = ToPrivateOwnerPtr(MoveTemp(*this));
	if (!ExistingOwner || ExistingOwner->IsUniqueOwnedMutable())
	{
		return FUniqueBuffer(MoveTemp(ExistingOwner));
	}
	else
	{
		return FUniqueBuffer::Clone(ExistingOwner->GetData(), ExistingOwner->GetSize());
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FWeakSharedBuffer::FWeakSharedBuffer(const FSharedBuffer& Buffer)
	: Owner(ToPrivateOwnerPtr(Buffer))
{
}

FWeakSharedBuffer& FWeakSharedBuffer::operator=(const FSharedBuffer& Buffer)
{
	Owner = ToPrivateOwnerPtr(Buffer);
	return *this;
}

void FWeakSharedBuffer::Reset()
{
	Owner.Reset();
}

FSharedBuffer FWeakSharedBuffer::Pin() const
{
	return FSharedBuffer(Owner);
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
