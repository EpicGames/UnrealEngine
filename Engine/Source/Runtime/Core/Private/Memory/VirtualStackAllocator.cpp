// Copyright Epic Games, Inc. All Rights Reserved.

#include "Memory/VirtualStackAllocator.h"

#include "HAL/LowLevelMemTracker.h"
#include "HAL/PlatformMemory.h"
#include "HAL/PlatformProcess.h"
#include "Templates/AlignmentTemplates.h"
#include "AutoRTFM.h"

LLM_DEFINE_TAG(VirtualStackAllocator);

template<typename T> static inline T* OffsetPointer(T* Start, size_t Offset)
{
    return reinterpret_cast<T*>(reinterpret_cast<intptr_t>(Start) + Offset);
}

template<typename T, typename U> static inline ptrdiff_t PointerDifference(T* End, U* Start)
{
    return static_cast<ptrdiff_t>(reinterpret_cast<intptr_t>(End) - reinterpret_cast<intptr_t>(Start));
}

FScopedStackAllocatorBookmark::~FScopedStackAllocatorBookmark()
{
    if (RestorePointer != nullptr)
    {
        check(Owner != nullptr);
        Owner->Free(RestorePointer);
    }
}

UE_AUTORTFM_NOAUTORTFM // Virtual stacks should be created outside of the AutoRTFM transaction.
FVirtualStackAllocator::FVirtualStackAllocator(size_t RequestedStackSize, EVirtualStackAllocatorDecommitMode Mode)
    : PageSize(FPlatformMemory::FPlatformVirtualMemoryBlock::GetCommitAlignment()),
	DecommitMode(Mode)
{
    TotalReservationSize = Align(RequestedStackSize, PageSize);
    
	if (TotalReservationSize > 0)
	{
		VirtualMemory = FPlatformMemory::FPlatformVirtualMemoryBlock::AllocateVirtual(TotalReservationSize);
		NextUncommittedPage = VirtualMemory.GetVirtualPointer();
		NextAllocationStart = NextUncommittedPage;
		RecentHighWaterMark = NextUncommittedPage;

#if PLATFORM_HAS_ASAN_INCLUDE && !UE_AUTORTFM // Disable poisoning when running with AutoRTFM, as poisoning cannot be safely rolled back.
		ASAN_POISON_MEMORY_REGION(NextUncommittedPage, TotalReservationSize);
#endif
	}
}

UE_AUTORTFM_NOAUTORTFM // Virtual stacks should be destroyed outside of the AutoRTFM transaction.
FVirtualStackAllocator::~FVirtualStackAllocator()
{
	check(GetAllocatedBytes() == 0);
	if (NextUncommittedPage != nullptr)
	{
		VirtualMemory.FreeVirtual();
	}
}

void* FVirtualStackAllocator::Allocate(size_t Size, size_t Alignment)
{
	void* const AllocationStart = Align(NextAllocationStart, Alignment);
	if (Size > 0)
	{
		void* const AllocationEnd = OffsetPointer(AllocationStart, Size);
		void* const UsableMemoryEnd = OffsetPointer(VirtualMemory.GetVirtualPointer(), TotalReservationSize - PageSize);

		if (AllocationEnd > UsableMemoryEnd)
		{
			FPlatformMemory::OnOutOfMemory(Size, Alignment);
		}

		// VirtualMemory, NextUncommittedPage and RecentHighWaterMark must never be modified from the closed in AutoRTFM.
 		UE_AUTORTFM_OPEN
		{
			// After the high water mark is established, needing to commit pages should be rare.
			if (UNLIKELY(AllocationEnd > NextUncommittedPage))
			{
				// We need to commit some more pages. Let's see how many
				uintptr_t RequiredAdditionalCommit = PointerDifference(AllocationEnd, NextUncommittedPage);
				// CommitByPtr doesn't round up the size for you
				size_t SizeToCommit = Align(RequiredAdditionalCommit, PageSize);
				// We must commit new pages in the open, and pages will (harmlessly) remain committed even if the AutoRTFM transaction is aborted.
				VirtualMemory.CommitByPtr(NextUncommittedPage, SizeToCommit);
 
				LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelAlloc(ELLMTracker::Default, NextUncommittedPage, SizeToCommit, LLM_TAG_NAME(VirtualStackAllocator)));
 
				NextUncommittedPage = Align(AllocationEnd, PageSize);
			};

			if ((char*)AllocationEnd > (char*)RecentHighWaterMark)
			{
				RecentHighWaterMark = Align(AllocationEnd, PageSize);
			}
		};

		NextAllocationStart = AllocationEnd;

#if PLATFORM_HAS_ASAN_INCLUDE && !UE_AUTORTFM // Disable poisoning when running with AutoRTFM, as poisoning cannot be safely rolled back.
		ASAN_UNPOISON_MEMORY_REGION(AllocationStart, Size);
#endif
	}
 
    return AllocationStart;
}

UE_AUTORTFM_ALWAYS_OPEN // VirtualMemory, NextUncommittedPage and RecentHighWaterMark must never be modified from the closed in AutoRTFM.
void FVirtualStackAllocator::DecommitUnusedPages()
{
	// We can only decommit unused pages when the allocator is empty.
	check(NextAllocationStart == VirtualMemory.GetVirtualPointer());

	if (DecommitMode == EVirtualStackAllocatorDecommitMode::AllOnStackEmpty)
	{
		VirtualMemory.Decommit();
		NextUncommittedPage = VirtualMemory.GetVirtualPointer();
	}
	else if (DecommitMode == EVirtualStackAllocatorDecommitMode::ExcessOnStackEmpty)
	{
		// In this mode, each time we get down to zero memory in use we consider decommitting some of the memory above the most recent high water mark
		ptrdiff_t AmountToFree = (intptr_t)NextUncommittedPage - (intptr_t)RecentHighWaterMark;

		// We will only decommit memory if it would free up at least 25% of the current commit. This helps prevent us from thrashing pages if our
		// memory usage is consistant but not exactly constant and ensures we only pay to decommit if it will actually result in a significant savings
		ptrdiff_t MinimumToDecommit = PointerDifference(NextUncommittedPage, VirtualMemory.GetVirtualPointer()) / 4;
		if (AmountToFree > MinimumToDecommit)
		{
			// We have used less memory this time than the last time, decommit the excess
			VirtualMemory.DecommitByPtr(RecentHighWaterMark, AmountToFree);
			NextUncommittedPage = RecentHighWaterMark;
		}
	}
	RecentHighWaterMark = VirtualMemory.GetVirtualPointer();
}
