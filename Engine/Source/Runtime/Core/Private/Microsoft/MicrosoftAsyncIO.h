// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_SKIP - Not included directly

#include "ProfilingDebugging/PlatformFileTrace.h"

PRAGMA_DISABLE_UNSAFE_TYPECAST_WARNINGS

class FMicrosoftReadRequest;
class FMicrosoftAsyncReadFileHandle;

#if !defined(USE_WINAPI_CREATEFILE2)
	#define USE_WINAPI_CREATEFILE2 0
#endif

class FMicrosoftReadRequestWorker : public FNonAbandonableTask
{
	FMicrosoftReadRequest& ReadRequest;
public:
	FMicrosoftReadRequestWorker(FMicrosoftReadRequest* InReadRequest)
		: ReadRequest(*InReadRequest)
	{
	}
	void DoWork();
	FORCEINLINE TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FMicrosoftReadRequestWorker, STATGROUP_ThreadPoolAsyncTasks);
	}
};

extern CORE_API TLockFreePointerListUnordered<void, PLATFORM_CACHE_LINE_SIZE> MicrosoftAsyncIOEventPool;

HANDLE GetIOPooledEvent()
{
	void *Popped = MicrosoftAsyncIOEventPool.Pop();
	HANDLE Result = Popped ? (HANDLE)UPTRINT(Popped) : INVALID_HANDLE_VALUE;
	if (Result == INVALID_HANDLE_VALUE)
	{
		Result = CreateEvent(nullptr, true, 0, nullptr);
		check((void*)(UPTRINT)Result); //awkwardly using void* to store handles, hope we don't ever have a zero handle :)
	}
	return Result;
}

void FreeIOPooledEvent(HANDLE ToFree)
{
	check(ToFree != INVALID_HANDLE_VALUE && (void*)(UPTRINT)ToFree); //awkwardly using void* to store handles, hope we don't ever have a zero handle :)
	ResetEvent(ToFree);
	MicrosoftAsyncIOEventPool.Push((void*)(UPTRINT)ToFree);
}


class FMicrosoftReadRequest : public IAsyncReadRequest
{
	FAsyncTask<FMicrosoftReadRequestWorker>* Task;
	FMicrosoftAsyncReadFileHandle* Owner;
	int64 Offset;
	int64 BytesToRead;
	int64 FileSize;
	HANDLE FileHandle;
	EAsyncIOPriorityAndFlags PriorityAndFlags;
	uint8* TempMemory;
	int64 AlignedOffset;
	int64 AlignedBytesToRead;
	OVERLAPPED OverlappedIO;
public:
	FMicrosoftReadRequest(FMicrosoftAsyncReadFileHandle* InOwner, FAsyncFileCallBack* CompleteCallback, uint8* InUserSuppliedMemory, int64 InOffset, int64 InBytesToRead, int64 InFileSize, HANDLE InHandle, EAsyncIOPriorityAndFlags InPriorityAndFlags)
		: IAsyncReadRequest(CompleteCallback, false, InUserSuppliedMemory)
		, Task(nullptr)
		, Owner(InOwner)
		, Offset(InOffset)
		, BytesToRead(InBytesToRead)
		, FileSize(InFileSize)
		, FileHandle(InHandle)
		, PriorityAndFlags(InPriorityAndFlags)
		, TempMemory(nullptr)
	{
		FMemory::Memzero(OverlappedIO);
		OverlappedIO.hEvent = INVALID_HANDLE_VALUE;
		check(Offset >= 0 && BytesToRead > 0);
		if (BytesToRead == MAX_int64)
		{
			BytesToRead = FileSize - Offset;
			check(BytesToRead > 0);
		}
		AlignedOffset = Offset;
		AlignedBytesToRead = BytesToRead;
		if (CheckForPrecache())
		{
			SetComplete();
		}
		else
		{
			AlignedOffset = AlignDown(Offset, 4096);
			AlignedBytesToRead = Align(Offset + BytesToRead, 4096) - AlignedOffset;
			check(AlignedOffset >= 0 && AlignedBytesToRead > 0);

			bool bMemoryHasBeenAcquired = bUserSuppliedMemory;
			if (bUserSuppliedMemory && (AlignedOffset != Offset || AlignedBytesToRead != BytesToRead))
			{
				static int32 NumMessages = 0;
				if (NumMessages < 10)
				{
					NumMessages++;
					UE_LOGF(LogTemp, Log, "FMicrosoftReadRequest request was not aligned. This is expected with loose files, but not a pak file.");
				}
				else if (NumMessages == 10)
				{
					NumMessages++;
					UE_LOGF(LogTemp, Log, "LAST NOTIFICATION THIS RUN: FMicrosoftReadRequest request was not aligned.");
				}
				TempMemory = (uint8*)FMemory::Malloc(AlignedBytesToRead);
				INC_MEMORY_STAT_BY(STAT_AsyncFileMemory, AlignedBytesToRead);
			}
			else if (!bMemoryHasBeenAcquired)
			{
				check(!Memory);
				Memory = (uint8*)FMemory::Malloc(AlignedBytesToRead);
				INC_MEMORY_STAT_BY(STAT_AsyncFileMemory, AlignedBytesToRead);
			}
			check(Memory);
			uint32 NumRead = 0;

			if (Offset + BytesToRead > FileSize || AlignedOffset < 0 || AlignedBytesToRead < 1)
			{
				UE_LOGF(LogTemp, Fatal, "FMicrosoftReadRequest bogus request Offset = %lld BytesToRead = %lld AlignedOffset = %lld AlignedBytesToRead = %lld FileSize = %lld File = %ls", Offset, BytesToRead, AlignedOffset, AlignedBytesToRead, FileSize, GetFileNameForErrorMessagesAndPanicRetry());
			}

			{
				ULARGE_INTEGER LI;
				LI.QuadPart = AlignedOffset;
				OverlappedIO.Offset = LI.LowPart;
				OverlappedIO.OffsetHigh = LI.HighPart;
			}
			OverlappedIO.hEvent = GetIOPooledEvent();
			TRACE_PLATFORMFILE_BEGIN_READ(&OverlappedIO, FileHandle, AlignedOffset, AlignedBytesToRead);
			if (!ReadFile(FileHandle, TempMemory ? TempMemory : Memory, AlignedBytesToRead, (LPDWORD)&NumRead, &OverlappedIO))
			{
				uint32 ErrorCode = GetLastError();
				if (ErrorCode != ERROR_IO_PENDING)
				{
					UE_LOGF(LogTemp, Fatal, "FMicrosoftReadRequest ReadFile Failed! Error code = %x", ErrorCode);
				}
			}

			Task = new FAsyncTask<FMicrosoftReadRequestWorker>(this);
			Start();
		}
	}
	virtual ~FMicrosoftReadRequest();

	bool CheckForPrecache();
	const TCHAR* GetFileNameForErrorMessagesAndPanicRetry();

	void PerformRequest()
	{

#if 0
		if (!HasOverlappedIoCompleted(&OverlappedIO))
		{
			WaitForSingleObject(OverlappedIO.hEvent, 0);
		}
#endif
		check(AlignedOffset <= Offset);
		uint32 BytesRead = 0;

		bool bFailed = false;
		FString FailedMessage;

		extern bool GTriggerFailedMicrosoftRead;

		if (GTriggerFailedMicrosoftRead || !GetOverlappedResult(FileHandle, &OverlappedIO, (LPDWORD)&BytesRead, TRUE))
		{
			TRACE_PLATFORMFILE_END_READ(&OverlappedIO, 0);
			GTriggerFailedMicrosoftRead = false;
			uint32 ErrorCode = GetLastError();
			FailedMessage = FString::Printf(TEXT("FMicrosoftReadRequest GetOverlappedResult Code = %x Offset = %lld Size = %lld FileSize = %lld File = %s"), ErrorCode, AlignedOffset, AlignedBytesToRead, FileSize, GetFileNameForErrorMessagesAndPanicRetry());
			bFailed = true;
		}
		else
		{
			TRACE_PLATFORMFILE_END_READ(&OverlappedIO, BytesRead);
		}
		if (!bFailed && int64(BytesRead) < BytesToRead + (Offset - AlignedOffset))
		{
			uint32 ErrorCode = GetLastError();
			FailedMessage = FString::Printf(TEXT("FMicrosoftReadRequest Short Read Code = %x BytesRead = %lld Offset = %lld AlignedOffset = %lld BytesToRead = %lld Size = %lld File = %s"), ErrorCode, int64(BytesRead), Offset, AlignedOffset, BytesToRead, FileSize, GetFileNameForErrorMessagesAndPanicRetry());
			bFailed = true;
		}

		if (bFailed)
		{
			UE_LOGF(LogTemp, Error, "Bad read, retrying %ls", *FailedMessage);

			for (int32 Try = 0; bFailed && Try < 10; Try++)
			{
#if USE_WINAPI_CREATEFILE2
				DWORD  Access = GENERIC_READ;
				DWORD  WinFlags = FILE_SHARE_READ;
				DWORD  Create = OPEN_EXISTING;
				CREATEFILE2_EXTENDED_PARAMETERS Params = { 0 };
				Params.dwSize = sizeof(CREATEFILE2_EXTENDED_PARAMETERS);
				Params.dwFileAttributes = FILE_ATTRIBUTE_READONLY;
				Params.dwFileFlags = FILE_FLAG_NO_BUFFERING;
				Params.dwSecurityQosFlags = SECURITY_ANONYMOUS;
				HANDLE Handle = CreateFile2(GetFileNameForErrorMessagesAndPanicRetry(), Access, WinFlags, Create, &Params);
#else
				uint32  Access = GENERIC_READ;
				uint32  WinFlags = FILE_SHARE_READ;
				uint32  Create = OPEN_EXISTING;
				HANDLE Handle = CreateFileW(GetFileNameForErrorMessagesAndPanicRetry(), Access, WinFlags, NULL, Create, FILE_ATTRIBUTE_NORMAL, NULL);
#endif
				if (Handle != INVALID_HANDLE_VALUE )
				{
					ULARGE_INTEGER LI;
					LI.QuadPart = AlignedOffset;
					CA_SUPPRESS(6001); // warning C6001: Using uninitialized memory '*Handle'.
					if (SetFilePointer(Handle, LI.LowPart, (PLONG)&LI.HighPart, FILE_BEGIN) == INVALID_SET_FILE_POINTER)
					{
						uint32 ErrorCode = GetLastError();
						UE_LOGF(LogTemp, Error, "Failed to seek for retry. %x", ErrorCode );
					}
					else if (ReadFile(Handle, TempMemory ? TempMemory : Memory, AlignedBytesToRead, (LPDWORD)&BytesRead, nullptr) &&
						!(int64(BytesRead) < BytesToRead + (Offset - AlignedOffset)))
					{
						bFailed = false;
					}
					else
					{
						uint32 ErrorCode = GetLastError();
						UE_LOGF(LogTemp, Error, "Failed to read for retry. %x", ErrorCode );
					}
					CloseHandle(Handle);
				}
				else
				{
					uint32 ErrorCode = GetLastError();
					UE_LOGF(LogTemp, Error, "Failed to open handle for retry. %x", ErrorCode );
				}
				if (bFailed && Try < 9)
				{
					FPlatformProcess::Sleep(.2f);
				}
			}
		}
		if (bFailed)
		{
			UE_LOGF(LogTemp, Fatal, "Unable to recover from a bad read: %ls", *FailedMessage);
		}

		FinalizeReadAndSetComplete();
	}

	void FinalizeReadAndSetComplete()
	{
		check(Memory);
		if (TempMemory)
		{
			FMemory::Memcpy(Memory, TempMemory + (Offset - AlignedOffset), BytesToRead);
			FMemory::Free(TempMemory);
			TempMemory = nullptr;
			DEC_MEMORY_STAT_BY(STAT_AsyncFileMemory, AlignedBytesToRead);
		}
		else if (AlignedOffset != Offset)
		{
			FMemory::Memmove(Memory, Memory + (Offset - AlignedOffset), BytesToRead);
		}
		SetComplete();
	}

	uint8* GetContainedSubblock(uint8* UserSuppliedMemory, int64 InOffset, int64 InBytesToRead)
	{
		if (InOffset >= Offset && InOffset + InBytesToRead <= Offset + BytesToRead &&
			this->PollCompletion() && Memory)
		{
			check(Memory);
			if (!UserSuppliedMemory)
			{
				UserSuppliedMemory = (uint8*)FMemory::Malloc(InBytesToRead);
				INC_MEMORY_STAT_BY(STAT_AsyncFileMemory, InBytesToRead);
			}
			FMemory::Memcpy(UserSuppliedMemory, Memory + InOffset - Offset, InBytesToRead);
			return UserSuppliedMemory;
		}
		return nullptr;
	}

	void Start()
	{
		if (FPlatformProcess::SupportsMultithreading())
		{
			Task->StartBackgroundTask(GIOThreadPool);
		}
		else
		{
			Task->StartSynchronousTask();
			WaitCompletionImpl(0.0f); // might as well finish it now
		}
	}

	virtual void WaitCompletionImpl(float TimeLimitSeconds) override
	{
		if (Task)
		{
			bool bResult;
			if (TimeLimitSeconds <= 0.0f)
			{
				Task->EnsureCompletion();
				bResult = true;
			}
			else
			{
				bResult = Task->WaitCompletionWithTimeout(TimeLimitSeconds);
			}
			if (bResult)
			{
				check(bCompleteAndCallbackCalled);
				delete Task;
				Task = nullptr;
			}
		}
	}

	virtual void CancelImpl() override
	{
		// no cancel support
	}

	virtual void ReleaseMemoryOwnershipImpl() override
	{
		DEC_MEMORY_STAT_BY(STAT_AsyncFileMemory, BytesToRead);
	}
};

void FMicrosoftReadRequestWorker::DoWork()
{
	ReadRequest.PerformRequest();
}

class FMicrosoftSizeRequest : public IAsyncReadRequest
{
public:
	FMicrosoftSizeRequest(FAsyncFileCallBack* CompleteCallback, int64 InFileSize)
		: IAsyncReadRequest(CompleteCallback, true, nullptr)
	{
		Size = InFileSize;
		SetComplete();
	}

	virtual void WaitCompletionImpl(float TimeLimitSeconds) override
	{
		// Even though SetComplete called in the constructor and sets bCompleteAndCallbackCalled=true, we still need to implement WaitComplete as
		// the CompleteCallback can end up starting async tasks that can overtake the constructor execution and need to wait for the constructor to finish.
		while (!*(volatile bool*)&bCompleteAndCallbackCalled);
	}

	virtual void CancelImpl() override
	{
	}

	virtual void ReleaseMemoryOwnershipImpl() override
	{
	}
};

class FMicrosoftFailedRequest : public IAsyncReadRequest
{
public:
	FMicrosoftFailedRequest(FAsyncFileCallBack* CompleteCallback)
		: IAsyncReadRequest(CompleteCallback, false, nullptr)
	{
		SetComplete();
	}

	virtual void WaitCompletionImpl(float TimeLimitSeconds) override
	{
		// Even though SetComplete called in the constructor and sets bCompleteAndCallbackCalled=true, we still need to implement WaitComplete as
		// the CompleteCallback can end up starting async tasks that can overtake the constructor execution and need to wait for the constructor to finish.
		while (!*(volatile bool*)&bCompleteAndCallbackCalled);
	}

	virtual void CancelImpl() override
	{
	}

	virtual void ReleaseMemoryOwnershipImpl() override
	{
	}
};


class FMicrosoftAsyncReadFileHandle final : public IAsyncReadFileHandle
{
public:
	HANDLE FileHandle;
	int64 FileSize;
	FString FileNameForErrorMessagesAndPanicRetry;
private:
	TArray<FMicrosoftReadRequest*> LiveRequests; // linear searches could be improved

	FCriticalSection LiveRequestsCritical;
	FCriticalSection HandleCacheCritical;
public:

	FMicrosoftAsyncReadFileHandle(HANDLE InFileHandle, const TCHAR* InFileNameForErrorMessagesAndPanicRetry)
		: FileHandle(InFileHandle)
		, FileSize(-1)
		, FileNameForErrorMessagesAndPanicRetry(InFileNameForErrorMessagesAndPanicRetry)
	{
		if (FileHandle != INVALID_HANDLE_VALUE)
		{
			LARGE_INTEGER LI;
			GetFileSizeEx(FileHandle, &LI);
			FileSize = LI.QuadPart;
		}
	}
	~FMicrosoftAsyncReadFileHandle()
	{
#if DO_CHECK
		FScopeLock Lock(&LiveRequestsCritical);
		check(!LiveRequests.Num()); // must delete all requests before you delete the handle
#endif
		TRACE_PLATFORMFILE_BEGIN_CLOSE(FileHandle);
		BOOL CloseResult = CloseHandle(FileHandle);
#if PLATFORMFILETRACE_ENABLED
		if (CloseResult)
		{
			TRACE_PLATFORMFILE_END_CLOSE(FileHandle);
		}
		else
		{
			TRACE_PLATFORMFILE_FAIL_CLOSE(FileHandle);
		}
#else
		(void)CloseResult;
#endif
	}
	void RemoveRequest(FMicrosoftReadRequest* Req)
	{
		FScopeLock Lock(&LiveRequestsCritical);
		verify(LiveRequests.Remove(Req) == 1);
	}
	uint8* GetPrecachedBlock(uint8* UserSuppliedMemory, int64 InOffset, int64 InBytesToRead)
	{
		FScopeLock Lock(&LiveRequestsCritical);
		uint8* Result = nullptr;
		for (FMicrosoftReadRequest* Req : LiveRequests)
		{
			Result = Req->GetContainedSubblock(UserSuppliedMemory, InOffset, InBytesToRead);
			if (Result)
			{
				break;
			}
		}
		return Result;
	}
	virtual IAsyncReadRequest* SizeRequest(FAsyncFileCallBack* CompleteCallback = nullptr) override
	{
		return new FMicrosoftSizeRequest(CompleteCallback, FileSize);
	}
	virtual IAsyncReadRequest* ReadRequest(int64 Offset, int64 BytesToRead, EAsyncIOPriorityAndFlags PriorityAndFlags = AIOP_Normal, FAsyncFileCallBack* CompleteCallback = nullptr, uint8* UserSuppliedMemory = nullptr) override
	{
		if (FileHandle != INVALID_HANDLE_VALUE)
		{
			FMicrosoftReadRequest* Result = new FMicrosoftReadRequest(this, CompleteCallback, UserSuppliedMemory, Offset, BytesToRead, FileSize, FileHandle, PriorityAndFlags);
			if (PriorityAndFlags & AIOP_FLAG_PRECACHE) // only precache requests are tracked for possible reuse
			{
				FScopeLock Lock(&LiveRequestsCritical);
				LiveRequests.Add(Result);
			}
			return Result;
		}
		return new FMicrosoftFailedRequest(CompleteCallback);
	}
};

FMicrosoftReadRequest::~FMicrosoftReadRequest()
{
	if (Task)
	{
		Task->EnsureCompletion(); // if the user polls, then we might never actual sync completion of the task until now, this will almost always be done, however we need to be sure the task is clear
		delete Task;
	}
	if (OverlappedIO.hEvent != INVALID_HANDLE_VALUE)
	{
		FreeIOPooledEvent(OverlappedIO.hEvent);
		OverlappedIO.hEvent = INVALID_HANDLE_VALUE;
	}
	if (Memory)
	{
		// this can happen with a race on cancel, it is ok, they didn't take the memory, free it now
		if (!bUserSuppliedMemory)
		{
			DEC_MEMORY_STAT_BY(STAT_AsyncFileMemory, BytesToRead);
			FMemory::Free(Memory);
		}
		Memory = nullptr;
	}
	if (TempMemory)
	{
		DEC_MEMORY_STAT_BY(STAT_AsyncFileMemory, AlignedBytesToRead);
		FMemory::Free(TempMemory);
		TempMemory = nullptr;
	}
	if (PriorityAndFlags & AIOP_FLAG_PRECACHE) // only precache requests are tracked for possible reuse
	{
		Owner->RemoveRequest(this);
	}
	Owner = nullptr;
}

bool FMicrosoftReadRequest::CheckForPrecache()
{
	if ((PriorityAndFlags & AIOP_FLAG_PRECACHE) == 0)  // only non-precache requests check for existing blocks to copy from 
	{
		check(!Memory || bUserSuppliedMemory);
		uint8* Result = Owner->GetPrecachedBlock(Memory, Offset, BytesToRead);
		if (Result)
		{
			check(!bUserSuppliedMemory || Memory == Result);
			Memory = Result;
			return true;
		}
	}
	return false;
}

const TCHAR* FMicrosoftReadRequest::GetFileNameForErrorMessagesAndPanicRetry()
{
	return *Owner->FileNameForErrorMessagesAndPanicRetry;
}

PRAGMA_RESTORE_UNSAFE_TYPECAST_WARNINGS
