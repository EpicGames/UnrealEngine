// Copyright Epic Games, Inc. All Rights Reserved.

#include "Microsoft/MicrosoftPlatformCrashContext.h"
#include "HAL/ThreadManager.h"

#include "Microsoft/AllowMicrosoftPlatformTypes.h"
	#include <psapi.h>
#include "Microsoft/HideMicrosoftPlatformTypes.h"

void FMicrosoftPlatformCrashContext::CaptureAllThreadContexts()
{
#if PLATFORM_SUPPORTS_ALL_THREAD_BACKTRACES
	FThreadManager::Get().ForEachThreadStackBackTrace(
		[this](uint32 ThreadId, const TCHAR* ThreadName, const TConstArrayView<uint64>& StackTrace)
		{
			AddPortableThreadCallStack(ThreadId, ThreadName, StackTrace.GetData(), StackTrace.Num());
			return true;
		});
#endif
}

void FMicrosoftPlatformCrashContext::AddPortableThreadCallStacks(TConstArrayView<FThreadCallStack> Threads)
{
	FModuleHandleArray ProcModuleHandles;
	GetProcModuleHandles(ProcessHandle, ProcModuleHandles);

	ThreadCallStacks.Reserve(ThreadCallStacks.Num() + Threads.Num());
	for (const FThreadCallStack& InThread : Threads)
	{
		FThreadStackFrames Thread;
		Thread.ThreadId = InThread.ThreadId;
		Thread.ThreadName = FString(InThread.ThreadName);
		ConvertProgramCountersToStackFrames(ProcessHandle, ProcModuleHandles, InThread.StackFrames.GetData(), InThread.StackFrames.Num(), Thread.StackFrames);
		ThreadCallStacks.Push(Thread);
	}
}

void FMicrosoftPlatformCrashContext::AddPortableThreadCallStack(uint32 ThreadId, const TCHAR* ThreadName, const uint64* StackFrames, int32 NumStackFrames)
{
	AddPortableThreadCallStacks({{ MakeArrayView(StackFrames, NumStackFrames), ThreadName, ThreadId }});
}

void FMicrosoftPlatformCrashContext::SetPortableCallStack(const uint64* StackTrace, int32 StackTraceDepth)
{
	FModuleHandleArray ProcessModuleHandles;
	GetProcModuleHandles(ProcessHandle, ProcessModuleHandles);
	ConvertProgramCountersToStackFrames(ProcessHandle, ProcessModuleHandles, StackTrace, StackTraceDepth, CallStack);
}

void FMicrosoftPlatformCrashContext::GetProcModuleHandles(const FProcHandle& ProcessHandle, FModuleHandleArray& OutHandles)
{
	// Most but not all Microsoft platforms have acess to EnumProcessModules*
#if WINAPI_FAMILY_PARTITION(WINAPI_PARTITION_DESKTOP | WINAPI_PARTITION_SYSTEM | WINAPI_PARTITION_GAMES)
	// Get all the module handles for the current process. Each module handle is its base address.
	for (;;)
	{
		DWORD BufferSize = OutHandles.Num() * sizeof(HMODULE);
		DWORD RequiredBufferSize = 0;
		if (!EnumProcessModulesEx(ProcessHandle.IsValid() ? ProcessHandle.Get() : GetCurrentProcess(), (HMODULE*)OutHandles.GetData(), BufferSize, &RequiredBufferSize, LIST_MODULES_ALL))
		{
			// We do not want partial set of modules in case this fails.
			OutHandles.Empty();
			return;
		}
		if (RequiredBufferSize <= BufferSize)
		{
			break;
		}
		OutHandles.SetNum(RequiredBufferSize / sizeof(HMODULE));
	}
	// Sort the handles by address. This allows us to do a binary search for the module containing an address.
	Algo::Sort(OutHandles);
#endif
}

void FMicrosoftPlatformCrashContext::ConvertProgramCountersToStackFrames(
	const FProcHandle& ProcessHandle,
	const FModuleHandleArray& SortedModuleHandles,
	const uint64* ProgramCounters,
	int32 NumPCs,
	TArray<FCrashStackFrame>& OutStackFrames)
{
	// Prepare the callstack buffer
	OutStackFrames.Reset(NumPCs);

	TCHAR Buffer[PLATFORM_MAX_FILEPATH_LENGTH_DEPRECATED];
	FString Unknown = "Unknown";

	HANDLE Process = ProcessHandle.IsValid() ? ProcessHandle.Get() : GetCurrentProcess();
	// Create the crash context
	for (int32 Idx = 0; Idx < NumPCs; ++Idx)
	{
		int32 ModuleIdx = Algo::UpperBound(SortedModuleHandles, (void*)ProgramCounters[Idx]) - 1;
		if (ModuleIdx < 0 || ModuleIdx >= SortedModuleHandles.Num())
		{
			OutStackFrames.Add(FCrashStackFrame(Unknown, 0, ProgramCounters[Idx]));
		}
		else
		{
			FStringView ModuleName = Unknown;
			if (GetModuleFileNameExW(Process, (HMODULE)SortedModuleHandles[ModuleIdx], Buffer, UE_ARRAY_COUNT(Buffer)) != 0)
			{
				ModuleName = Buffer;
				if (int32 CharIdx = 0; ModuleName.FindLastChar('\\', CharIdx))
				{
					ModuleName.RightChopInline(CharIdx +1);
				}
				if (int32 CharIdx = 0; ModuleName.FindLastChar('.', CharIdx))
				{
					ModuleName.LeftInline(CharIdx);
				}
			}

			uint64 BaseAddress = (uint64)SortedModuleHandles[ModuleIdx];
			uint64 Offset = ProgramCounters[Idx] - BaseAddress;
			OutStackFrames.Add(FCrashStackFrame(FString(ModuleName), BaseAddress, Offset));
		}
	}
}

bool FMicrosoftPlatformCrashContext::GetPlatformAllThreadContextsString(FString& OutStr) const
{
	for (const FThreadStackFrames& Thread : ThreadCallStacks)
	{
		AddThreadContextString(
			CrashedThreadId,
			Thread.ThreadId,
			Thread.ThreadName,
			Thread.StackFrames,
			OutStr
		);
	}
	return !OutStr.IsEmpty();
}

void FMicrosoftPlatformCrashContext::AddThreadContextString(
	uint32 CrashedThreadId,
	uint32 ThreadId,
	const FString& ThreadName,
	const TArray<FCrashStackFrame>& StackFrames,
	FString& OutStr)
{
	OutStr += TEXT("<Thread>");
	{
		OutStr += TEXT("<CallStack>");

		int32 MaxModuleNameLen = 0;
		for (const FCrashStackFrame& StFrame : StackFrames)
		{
			MaxModuleNameLen = FMath::Max(MaxModuleNameLen, StFrame.ModuleName.Len());
		}

		FString CallstackStr;
		for (const FCrashStackFrame& StFrame : StackFrames)
		{
			CallstackStr += FString::Printf(TEXT("%-*s 0x%016llx + %-16llx"), MaxModuleNameLen + 1, *StFrame.ModuleName, StFrame.BaseAddress, StFrame.Offset);
			CallstackStr += LINE_TERMINATOR;
		}
		AppendEscapedXMLString(OutStr, *CallstackStr);
		OutStr += TEXT("</CallStack>");
		OutStr += LINE_TERMINATOR;
	}
	OutStr += FString::Printf(TEXT("<IsCrashed>%s</IsCrashed>"), ThreadId == CrashedThreadId ? TEXT("true") : TEXT("false"));
	OutStr += LINE_TERMINATOR;
	// TODO: do we need thread register states?
	OutStr += TEXT("<Registers></Registers>");
	OutStr += LINE_TERMINATOR;
	OutStr += FString::Printf(TEXT("<ThreadID>%d</ThreadID>"), ThreadId);
	OutStr += LINE_TERMINATOR;
	OutStr += FString::Printf(TEXT("<ThreadName>%s</ThreadName>"), *ThreadName);
	OutStr += LINE_TERMINATOR;
	OutStr += TEXT("</Thread>");
	OutStr += LINE_TERMINATOR;
}
