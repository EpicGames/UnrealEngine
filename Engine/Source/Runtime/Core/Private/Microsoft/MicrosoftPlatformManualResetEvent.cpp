// Copyright Epic Games, Inc. All Rights Reserved.

#include "Microsoft/MicrosoftPlatformManualResetEvent.h"

#include "Async/Fundamental/Oversubscription.h"
#include "HAL/PlatformMath.h"
#include "Microsoft/WindowsHWrapper.h"

namespace UE::HAL::Private
{

FORCENOINLINE void FMicrosoftManualResetEvent::WaitSlow()
{
	LowLevelTasks::FOversubscriptionScope _;

	for (;;)
	{
		bool bLocalWait = bWait.load(std::memory_order_acquire);
		if (!bLocalWait)
		{
			return;
		}
		WaitOnAddress(&bWait, &bLocalWait, sizeof(bool), INFINITE);
	}
}

FORCENOINLINE bool FMicrosoftManualResetEvent::WaitForSlow(FMonotonicTimeSpan WaitTime)
{
	bool bLocalWait = bWait.load(std::memory_order_acquire);
	if (!bLocalWait || WaitTime <= FMonotonicTimeSpan::Zero())
	{
		return !bLocalWait;
	}

	LowLevelTasks::FOversubscriptionScope _;

	// Clamp to INFINITE. Test against INFINITE - 1 because of the ceiling operation.
	const double RawWaitMs = WaitTime.ToMilliseconds();
	const DWORD WaitMs = RawWaitMs > double(INFINITE - 1) ? INFINITE : DWORD(FPlatformMath::CeilToInt64(RawWaitMs));

	const bool bTimedOut = !WaitOnAddress(&bWait, &bLocalWait, sizeof(bool), WaitMs) && GetLastError() == ERROR_TIMEOUT;
	bLocalWait = bWait.load(std::memory_order_acquire);
	if (LIKELY(!bLocalWait || bTimedOut))
	{
		return !bLocalWait;
	}

	// Handle a spurious wake by waiting until the wait time has elapsed one more time because WaitUntilSlow
	// handles spurious wakes in a loop and avoids exceeding the originally requested wake time by more than
	// the typical variation due to scheduling imprecision.
	return WaitUntilSlow(FMonotonicTimePoint::Now() + WaitTime);
}

FORCENOINLINE bool FMicrosoftManualResetEvent::WaitUntilSlow(FMonotonicTimePoint WaitTime)
{
	FMonotonicTimeSpan WaitSpan = WaitTime - FMonotonicTimePoint::Now();
	LowLevelTasks::FOversubscriptionScope _(WaitSpan > FMonotonicTimeSpan::Zero());

	for (;;)
	{
		bool bLocalWait = bWait.load(std::memory_order_acquire);
		if (!bLocalWait || WaitSpan <= FMonotonicTimeSpan::Zero())
		{
			return !bLocalWait;
		}
		const DWORD WaitMs = WaitTime.IsInfinity() ? INFINITE : DWORD(FPlatformMath::CeilToInt64(WaitSpan.ToMilliseconds()));
		WaitOnAddress(&bWait, &bLocalWait, sizeof(bool), WaitMs);
		WaitSpan = WaitTime - FMonotonicTimePoint::Now();
	}
}

void FMicrosoftManualResetEvent::Notify()
{
	bWait.store(false, std::memory_order_release);
	WakeByAddressSingle((void*)&bWait);
}

} // UE::HAL::Private
