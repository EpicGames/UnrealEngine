// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/MonotonicTime.h"

#include <atomic>

namespace UE::HAL::Private
{

/** @see FGenericPlatformManualResetEvent */
class FMicrosoftManualResetEvent
{
public:
	FMicrosoftManualResetEvent() = default;
	FMicrosoftManualResetEvent(const FMicrosoftManualResetEvent&) = delete;
	FMicrosoftManualResetEvent& operator=(const FMicrosoftManualResetEvent&) = delete;

	inline void Reset()
	{
		bWait.store(true, std::memory_order_relaxed);
	}

	inline bool Poll()
	{
		return !bWait.load(std::memory_order_acquire);
	}

	inline void Wait()
	{
		if (bWait.load(std::memory_order_acquire))
		{
			WaitSlow();
		}
	}

	/**
	 * Waits for the wait time for Notify() to be called.
	 *
	 * Notify() may be called prior to WaitFor(), and this will return immediately in that case.
	 *
	 * NOTE: Windows will round the wait time to the nearest system tick and may coalesce timers
	 *       for power efficiency. These details can cause a timeout to elapse slightly early.
	 *
	 * @param WaitTime   Relative time after which waiting is canceled and the thread wakes.
	 * @return True if Notify() was called before the wait time elapsed, otherwise false.
	 */
	inline bool WaitFor(FMonotonicTimeSpan WaitTime)
	{
		return !bWait.load(std::memory_order_acquire) || WaitForSlow(WaitTime);
	}

	/**
	 * Waits until the wait time for Notify() to be called.
	 *
	 * Notify() may be called prior to WaitUntil(), and this will return immediately in that case.
	 *
	 * NOTE: Windows will round the wait time to the nearest system tick and may coalesce timers
	 *       for power efficiency. These details can cause a timeout to elapse slightly early.
	 *
	 * @param WaitTime   Absolute time after which waiting is canceled and the thread wakes.
	 * @return True if Notify() was called before the wait time elapsed, otherwise false.
	 */
	inline bool WaitUntil(FMonotonicTimePoint WaitTime)
	{
		return !bWait.load(std::memory_order_acquire) || WaitUntilSlow(WaitTime);
	}

	void Notify();

private:
	void WaitSlow();
	bool WaitForSlow(FMonotonicTimeSpan WaitTime);
	bool WaitUntilSlow(FMonotonicTimePoint WaitTime);

	std::atomic<bool> bWait = true;
};

} // UE::HAL::Private
