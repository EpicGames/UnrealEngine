// Copyright Epic Games, Inc. All Rights Reserved.

#include "Microsoft/MicrosoftPlatformStackWalk.h"

#include "Misc/Paths.h"
#include "CoreGlobals.h"

#include "Microsoft/AllowMicrosoftPlatformTypes.h"
	#include <psapi.h>
#include "Microsoft/HideMicrosoftPlatformTypes.h"

bool FMicrosoftPlatformStackWalk::ExtractInfoFromModule(void* ProcessHandle, void* Module, FStackWalkModuleInfo& OutInfo)
{
	MODULEINFO ModuleInfo = { 0 };
	WCHAR ModuleName[MAX_PATH] = { 0 };
	WCHAR ImageName[MAX_PATH] = { 0 };
	static_assert(sizeof(MODULEINFO) == 24, "Broken alignment for 64bit Windows include.");
	GetModuleInformation(ProcessHandle, (HMODULE)Module, &ModuleInfo, sizeof(ModuleInfo));
	GetModuleFileNameEx(ProcessHandle, (HMODULE)Module, ImageName, MAX_PATH);
	GetModuleBaseName(ProcessHandle, (HMODULE)Module, ModuleName, MAX_PATH);

	const uint8* ModuleBase = reinterpret_cast<const uint8*>(ModuleInfo.lpBaseOfDll);
	const auto* DosHeader = reinterpret_cast<const IMAGE_DOS_HEADER*>(ModuleBase);
	const auto* NtHeaders = reinterpret_cast<const IMAGE_NT_HEADERS*>(ModuleBase + DosHeader->e_lfanew);
	bool bValid = DosHeader->e_magic == IMAGE_DOS_SIGNATURE && NtHeaders->Signature == IMAGE_NT_SIGNATURE;
	if (!bValid)
	{
		return false;
	}

	const IMAGE_OPTIONAL_HEADER& OptionalHeader = NtHeaders->OptionalHeader;
	OutInfo.BaseOfImage = (uint64)ModuleInfo.lpBaseOfDll;
	OutInfo.ImageSize = OptionalHeader.SizeOfImage;
	OutInfo.TimeDateStamp = NtHeaders->FileHeader.TimeDateStamp;
	FString BaseModuleName = FPaths::GetBaseFilename(ModuleName);
	FCString::Strncpy(OutInfo.ModuleName, *BaseModuleName, 32);
	FCString::Strcpy(OutInfo.LoadedImageName, ImageName);
	FCString::Strcpy(OutInfo.ImageName, ImageName);

	// Find the guid and age of the binary, used to match debug files
	const IMAGE_DATA_DIRECTORY& DebugInfoEntry = OptionalHeader.DataDirectory[IMAGE_DIRECTORY_ENTRY_DEBUG];
	const auto* DebugEntries = (IMAGE_DEBUG_DIRECTORY*)(ModuleBase + DebugInfoEntry.VirtualAddress);
	for (uint32 i = 0, n = DebugInfoEntry.Size / sizeof(DebugEntries[0]); i < n; ++i)
	{
		const IMAGE_DEBUG_DIRECTORY& Entry = DebugEntries[i];
		if (Entry.Type == IMAGE_DEBUG_TYPE_CODEVIEW)
		{
			struct FCodeView7
			{
				uint32  Signature;
				uint32  Guid[4];
				uint32  Age;
			};

			if (Entry.SizeOfData < sizeof(FCodeView7))
			{
				continue;
			}

			const auto* CodeView7 = (FCodeView7*)(ModuleBase + Entry.AddressOfRawData);
			if (CodeView7->Signature != 'SDSR')
			{
				continue;
			}

			static_assert(sizeof(OutInfo.PdbSig70) == sizeof(uint32) * 4);
			FMemory::Memcpy(&OutInfo.PdbSig70, CodeView7->Guid, sizeof(OutInfo.PdbSig70));
			OutInfo.PdbAge = CodeView7->Age;
			break;
		}
	}

	return true;
}

int32 FMicrosoftPlatformStackWalk::CaptureStackTraceInternal(uint64* OutBacktrace, uint32 MaxDepth, void* InContext, void* InThreadHandle, uint32* OutDepth)
{
#if PLATFORM_CPU_X86_FAMILY || PLATFORM_WINDOWS_ARM64EC
	uint32 CurrentDepth = 0;
	HANDLE ThreadHandle = reinterpret_cast<HANDLE>(InThreadHandle);
	CONTEXT ContextCopy = *reinterpret_cast<PCONTEXT>(InContext);

	*OutDepth = 0;

	if (!ContextCopy.Rip || !ContextCopy.Rsp)
	{
		return EXCEPTION_EXECUTE_HANDLER;
	}

#if !PLATFORM_SEH_EXCEPTIONS_DISABLED
	__try
#endif
	{
		UNWIND_HISTORY_TABLE UnwindHistoryTable;
		RtlZeroMemory(&UnwindHistoryTable, sizeof(UNWIND_HISTORY_TABLE));

		for (; CurrentDepth < MaxDepth; ++CurrentDepth)
		{
			// If we reach an RIP of zero, this means that we've walked off the end
			// of the call stack and are done.
			if (!ContextCopy.Rip)
				break;

			OutBacktrace[CurrentDepth] = ContextCopy.Rip;
			*OutDepth = CurrentDepth;

			ULONG64 ImageBase = 0;
			if (PRUNTIME_FUNCTION RuntimeFunction = RtlLookupFunctionEntry(ContextCopy.Rip, &ImageBase, &UnwindHistoryTable); RuntimeFunction)
			{
				// Otherwise, call upon RtlVirtualUnwind to execute the unwind for us.
				KNONVOLATILE_CONTEXT_POINTERS NVContext;
				RtlZeroMemory(&NVContext, sizeof(KNONVOLATILE_CONTEXT_POINTERS));

				PVOID HandlerData = nullptr;
				ULONG64 EstablisherFrame = 0;

				RtlVirtualUnwind(
					0 /*UNW_FLAG_NHANDLER*/,
					ImageBase,
					ContextCopy.Rip,
					RuntimeFunction,
					&ContextCopy,
					&HandlerData,
					&EstablisherFrame,
					&NVContext);
			}
			else
			{
				// https://docs.microsoft.com/en-us/cpp/build/x64-calling-convention?view=msvc-170
				// Some functions are 'leaf' functions and do not have unwind info, they can be unwound
				// by simulating a return instruction.
				ContextCopy.Rip = (uint64)(*(uint64*)ContextCopy.Rsp);
				ContextCopy.Rsp += 8;
			}
		}
	}
#if !PLATFORM_SEH_EXCEPTIONS_DISABLED
	__except (EXCEPTION_EXECUTE_HANDLER)
	{
		// We need to catch any exceptions within this function so they don't get sent to
		// the engine's error handler, hence causing an infinite loop.
		return EXCEPTION_EXECUTE_HANDLER;
	}
#endif

	// NULL out remaining entries.
	for (; CurrentDepth < MaxDepth; CurrentDepth++)
	{
		OutBacktrace[CurrentDepth] = 0;
	}

#else
	UE_LOGF(LogCore, Fatal, "Stack trace via RtlLookupFunctionEntry only implemented for x86-64 Microsoft platforms.");
#endif
	return EXCEPTION_EXECUTE_HANDLER;
}