// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StringConv.h"
#include "CoreGlobals.h"
#include "CoreTypes.h"
#include "HAL/Event.h"
#include "HAL/LowLevelMemTracker.h"
#include "HAL/PlatformProcess.h"
#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "HAL/ThreadManager.h"
#include "ProfilingDebugging/MemoryTrace.h"
#include "Microsoft/WindowsHWrapper.h"

class FRunnable;

/**
 * This is the base interface for all runnable thread classes. It specifies the
 * methods used in managing its life cycle.
 */
class FRunnableThreadWin
	: public FRunnableThread
{
	/** The thread handle for the thread. */
	HANDLE Thread = 0;

	/**
	 * The thread entry point. Simply forwards the call on to the right
	 * thread main function
	 */
	static ::DWORD STDCALL _ThreadProc(LPVOID pThis)
	{
		check(pThis);
		auto* ThisThread = (FRunnableThreadWin*)pThis;
		FThreadManager::Get().AddThread(ThisThread->GetThreadID(), ThisThread);
		return ThisThread->GuardedRun();
	}

	/** Guarding works only if debugger is not attached or GAlwaysReportCrash is true. */
	uint32 GuardedRun();

	/**
	 * The real thread entry point. It calls the Init/Run/Exit methods on
	 * the runnable object
	 */
	uint32 Run();

public:
	~FRunnableThreadWin()
	{
		if (Thread)
		{
			Kill(true);
		}
	}

	virtual void SetThreadPriority(EThreadPriority NewPriority) override
	{
		ThreadPriority = NewPriority;

		::SetThreadPriority(Thread, TranslateThreadPriority(ThreadPriority));
	}

	virtual void Suspend(bool bShouldPause = true) override
	{
		check(Thread);
		if (bShouldPause == true)
		{
			SuspendThread(Thread);
		}
		else
		{
			ResumeThread(Thread);
		}
	}

	virtual bool Kill(bool bShouldWait) override
	{
		check(Thread && "Did you forget to call Create()?");
		bool bDidExitOK = true;

		// Let the runnable have a chance to stop without brute force killing
		if (Runnable)
		{
			Runnable->Stop();
		}

		if (bShouldWait == true)
		{
			// Wait indefinitely for the thread to finish.  IMPORTANT:  It's not safe to just go and
			// kill the thread with TerminateThread() as it could have a mutex lock that's shared
			// with a thread that's continuing to run, which would cause that other thread to
			// dead-lock.  
			//
			// This can manifest itself in code as simple as the synchronization
			// object that is used by our logging output classes

			WaitForSingleObject(Thread, INFINITE);
		}

		CloseHandle(Thread);
		Thread = NULL;

#if UE_MEMORY_TRACE_ENABLED || ENABLE_LOW_LEVEL_MEM_TRACKER
		const uint64 FakeAddress = uint64(this) | (1ull << 47);
		MemoryTrace_Free(FakeAddress);
		LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelFree(ELLMTracker::Default, (const void*)FakeAddress));
		LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelFree(ELLMTracker::Platform, (const void*)FakeAddress));
#endif // UE_MEMORY_TRACE_ENABLED || ENABLE_LOW_LEVEL_MEM_TRACKER

		return bDidExitOK;
	}

	virtual void WaitForCompletion() override
	{
		WaitForSingleObject(Thread, INFINITE);
	}

	virtual bool SetThreadAffinity(const FThreadAffinity& Affinity) override;

	static int TranslateThreadPriority(EThreadPriority Priority);
protected:

	virtual bool CreateInternal(FRunnable* InRunnable, const TCHAR* InThreadName,
		uint32 InStackSize = 0,
		EThreadPriority InThreadPri = TPri_Normal, uint64 InThreadAffinityMask = 0,
		EThreadCreateFlags InCreateFlags = EThreadCreateFlags::None) override
	{
		check(InRunnable);
		Runnable = InRunnable;
		ThreadAffinityMask = InThreadAffinityMask;

		// Create a sync event to guarantee the Init() function is called first
		ThreadInitSyncEvent = FPlatformProcess::GetSynchEventFromPool(true);

		ThreadName = InThreadName ? InThreadName : TEXT("Unnamed UE");
		ThreadPriority = InThreadPri;

		// Create the new thread
		{
#if UE_MEMORY_TRACE_ENABLED || ENABLE_LOW_LEVEL_MEM_TRACKER
			LLM_SCOPE(ELLMTag::ThreadStack);
			LLM_PLATFORM_SCOPE(ELLMTag::ThreadStackPlatform);
			const uint64 FakeAddress = uint64(this) | (1ull << 47);
			constexpr uint64 DefaultStackSize = 1024 * 1024; // 1 MiB, unless overridden in the .def file
			constexpr uint32 DefaultAlignment = 64 * 1024; // 64 KiB, typical system's allocation granularity
			// Size of zero indicates using default thread stack size.
			const uint64 ActualStackSize = (InStackSize == 0) ? DefaultStackSize : uint64(InStackSize);
			MemoryTrace_Alloc(FakeAddress, ActualStackSize, DefaultAlignment);
			MemoryTrace_MarkAllocAsHeap(FakeAddress, EMemoryTraceRootHeap::SystemMemory);
			MemoryTrace_Alloc(FakeAddress, ActualStackSize, DefaultAlignment);
			LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelAlloc(ELLMTracker::Default, (const void*)FakeAddress, ActualStackSize));
			LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelAlloc(ELLMTracker::Platform, (const void*)FakeAddress, ActualStackSize));
#endif // UE_MEMORY_TRACE_ENABLED || ENABLE_LOW_LEVEL_MEM_TRACKER

			// Create the thread as suspended, so we can ensure ThreadId is initialized and the thread manager knows about the thread before it runs.
			Thread = CreateThread(NULL, InStackSize, _ThreadProc, this, STACK_SIZE_PARAM_IS_A_RESERVATION | CREATE_SUSPENDED, (::DWORD*)&ThreadID);
		}

		// If it fails, clear all the vars
		if (Thread == NULL)
		{
#if UE_MEMORY_TRACE_ENABLED || ENABLE_LOW_LEVEL_MEM_TRACKER
			const uint64 FakeAddress = uint64(this) | (1ull << 47);
			MemoryTrace_Free(FakeAddress);
			LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelFree(ELLMTracker::Default, (const void*)FakeAddress));
			LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelFree(ELLMTracker::Platform, (const void*)FakeAddress));
#endif // UE_MEMORY_TRACE_ENABLED || ENABLE_LOW_LEVEL_MEM_TRACKER

			Runnable = nullptr;
		}
		else
		{
			// Set the priority before resuming the thread, since the thread itself could end up manipulating its own priority
			// and cause a race condition.
			ThreadPriority = TPri_Normal; // Set back to default in case any SetThreadPrio() impls compare against current value to reduce syscalls
			SetThreadPriority(InThreadPri);

			ResumeThread(Thread);

			// Let the thread start up
			ThreadInitSyncEvent->Wait(INFINITE);
		}

		// Cleanup the sync event
		FPlatformProcess::ReturnSynchEventToPool(ThreadInitSyncEvent);
		ThreadInitSyncEvent = nullptr;
		return Thread != NULL;
	}
};
