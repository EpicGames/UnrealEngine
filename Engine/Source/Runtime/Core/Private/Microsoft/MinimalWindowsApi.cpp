// Copyright Epic Games, Inc. All Rights Reserved.

#include "Microsoft/MinimalWindowsApi.h"
#include "Microsoft/WindowsHWrapper.h"

// Check that constants are what we expect them to be
static_assert(WINDOWS_MAX_PATH == MAX_PATH, "Value of WINDOWSAPI_MAX_PATH is not correct");
static_assert(WINDOWS_PF_COMPARE_EXCHANGE128 == PF_COMPARE_EXCHANGE128, "Value of WINDOWS_PF_COMPARE_EXCHANGE128 is not correct");

// Check that AllocTlsSlot() returns INDEX_NONE on failure
static_assert(static_cast<uint32>(INDEX_NONE) == TLS_OUT_OF_INDEXES, "TLS_OUT_OF_INDEXES is different from INDEX_NONE, change FWindowsPlatformTLS::AllocTlsSlot() implementation.");

// Check the size and alignment of the OVERLAPPED mirror
static_assert(sizeof(Windows::OVERLAPPED) == sizeof(OVERLAPPED), "Size of Windows::OVERLAPPED must match definition in Windows.h");
static_assert(__alignof(Windows::OVERLAPPED) == __alignof(OVERLAPPED), "Alignment of Windows::OVERLAPPED must match definition in Windows.h");

// Check the size and alignment of the CRITICAL_SECTION mirror
static_assert(sizeof(Windows::CRITICAL_SECTION) == sizeof(CRITICAL_SECTION), "Size of Windows::CRITICAL_SECTION must match definition in Windows.h");
static_assert(__alignof(Windows::CRITICAL_SECTION) == __alignof(CRITICAL_SECTION), "Alignment of Windows::CRITICAL_SECTION must match definition in Windows.h");

// Check the size and alignment of the LARGE_INTEGER mirror
static_assert(sizeof(Windows::LARGE_INTEGER) == sizeof(LARGE_INTEGER), "Size of Windows::LARGE_INTEGER must match definition in Windows.h");
static_assert(__alignof(Windows::LARGE_INTEGER) == __alignof(LARGE_INTEGER), "Alignment of Windows::LARGE_INTEGER must match definition in Windows.h");

#ifdef __clang__
namespace Windows
{
	MINIMAL_WINDOWS_API HMODULE WINAPI LoadLibraryW(LPCTSTR lpFileName)
	{
		return ::LoadLibraryW(lpFileName);
	}

	MINIMAL_WINDOWS_API BOOL WINAPI FreeLibrary(HMODULE hModule)
	{
		return ::FreeLibrary(hModule);
	}

	MINIMAL_WINDOWS_API void WINAPI InitializeCriticalSection(LPCRITICAL_SECTION lpCriticalSection)
	{
		::InitializeCriticalSection(lpCriticalSection);
	}

	MINIMAL_WINDOWS_API BOOL WINAPI InitializeCriticalSectionAndSpinCount(LPCRITICAL_SECTION lpCriticalSection, DWORD dwSpinCount)
	{
		return ::InitializeCriticalSectionAndSpinCount(lpCriticalSection, dwSpinCount);
	}

	MINIMAL_WINDOWS_API DWORD WINAPI SetCriticalSectionSpinCount(LPCRITICAL_SECTION lpCriticalSection, DWORD dwSpinCount)
	{
		return ::SetCriticalSectionSpinCount(lpCriticalSection, dwSpinCount);
	}

	MINIMAL_WINDOWS_API BOOL WINAPI TryEnterCriticalSection(LPCRITICAL_SECTION lpCriticalSection)
	{
		return ::TryEnterCriticalSection(lpCriticalSection);
	}

	MINIMAL_WINDOWS_API void WINAPI EnterCriticalSection(LPCRITICAL_SECTION lpCriticalSection)
	{
		::EnterCriticalSection(lpCriticalSection);
	}

	MINIMAL_WINDOWS_API void WINAPI LeaveCriticalSection(LPCRITICAL_SECTION lpCriticalSection)
	{
		::LeaveCriticalSection(lpCriticalSection);
	}

	MINIMAL_WINDOWS_API void WINAPI DeleteCriticalSection(LPCRITICAL_SECTION lpCriticalSection)
	{
		::DeleteCriticalSection(lpCriticalSection);
	}

	MINIMAL_WINDOWS_API void WINAPI InitializeSRWLock(PSRWLOCK SRWLock)
	{
		::InitializeSRWLock((::PSRWLOCK)SRWLock);
	}

	MINIMAL_WINDOWS_API void WINAPI AcquireSRWLockShared(PSRWLOCK SRWLock)
	{
		::AcquireSRWLockShared((::PSRWLOCK)SRWLock);
	}

	MINIMAL_WINDOWS_API BOOLEAN WINAPI TryAcquireSRWLockShared(PSRWLOCK SRWLock)
	{
		return ::TryAcquireSRWLockShared((::PSRWLOCK)SRWLock);
	}

	MINIMAL_WINDOWS_API BOOLEAN WINAPI TryAcquireSRWLockExclusive(PSRWLOCK SRWLock)
	{
		return ::TryAcquireSRWLockExclusive((::PSRWLOCK)SRWLock);
	}

	MINIMAL_WINDOWS_API void WINAPI ReleaseSRWLockShared(PSRWLOCK SRWLock)
	{
		::ReleaseSRWLockShared((::PSRWLOCK)SRWLock);
	}

	MINIMAL_WINDOWS_API void WINAPI AcquireSRWLockExclusive(PSRWLOCK SRWLock)
	{
		::AcquireSRWLockExclusive((::PSRWLOCK)SRWLock);
	}

	MINIMAL_WINDOWS_API void WINAPI ReleaseSRWLockExclusive(PSRWLOCK SRWLock)
	{
		::ReleaseSRWLockExclusive((::PSRWLOCK)SRWLock);
	}

	MINIMAL_WINDOWS_API BOOL WINAPI ConnectNamedPipe(HANDLE hNamedPipe, LPOVERLAPPED lpOverlapped)
	{
		return ::ConnectNamedPipe(hNamedPipe, lpOverlapped);
	}

	MINIMAL_WINDOWS_API BOOL WINAPI GetOverlappedResult(HANDLE hFile, LPOVERLAPPED lpOverlapped, LPDWORD lpNumberOfBytesTransferred, BOOL bWait)
	{
		return ::GetOverlappedResult(hFile, lpOverlapped, lpNumberOfBytesTransferred, bWait);
	}

	MINIMAL_WINDOWS_API BOOL WINAPI WriteFile(HANDLE hFile, LPCVOID lpBuffer, DWORD nNumberOfBytesToWrite, LPDWORD lpNumberOfBytesWritten, LPOVERLAPPED lpOverlapped)
	{
		return ::WriteFile(hFile, lpBuffer, nNumberOfBytesToWrite, lpNumberOfBytesWritten, lpOverlapped);
	}

	MINIMAL_WINDOWS_API BOOL WINAPI ReadFile(HANDLE hFile, LPVOID lpBuffer, DWORD nNumberOfBytesToRead, LPDWORD lpNumberOfBytesRead, LPOVERLAPPED lpOverlapped)
	{
		return ::ReadFile(hFile, lpBuffer, nNumberOfBytesToRead, lpNumberOfBytesRead, lpOverlapped);
	}

	MINIMAL_WINDOWS_API BOOL WINAPI CancelIoEx(HANDLE hFile, LPOVERLAPPED lpOverlapped)
	{
		return ::CancelIoEx(hFile, lpOverlapped);
	}

	MINIMAL_WINDOWS_API BOOL WINAPI QueryPerformanceCounter(LPLARGE_INTEGER Cycles)
	{
		return ::QueryPerformanceCounter(Cycles);
	}

	MINIMAL_WINDOWS_API DWORD WINAPI GetCurrentThreadId()
	{
		return ::GetCurrentThreadId();
	}

	MINIMAL_WINDOWS_API DWORD WINAPI TlsAlloc()
	{
		return ::TlsAlloc();
	}

	MINIMAL_WINDOWS_API LPVOID WINAPI TlsGetValue(DWORD dwTlsIndex)
	{
		return ::TlsGetValue(dwTlsIndex);
	}

	MINIMAL_WINDOWS_API BOOL WINAPI TlsSetValue(DWORD dwTlsIndex, LPVOID lpTlsValue)
	{
		return ::TlsSetValue(dwTlsIndex, lpTlsValue);
	}

	MINIMAL_WINDOWS_API BOOL WINAPI TlsFree(DWORD dwTlsIndex)
	{
		return ::TlsFree(dwTlsIndex);
	}

	MINIMAL_WINDOWS_API DWORD WINAPI FlsAlloc(PFLS_CALLBACK_FUNCTION lpCallback)
	{
		return ::FlsAlloc(lpCallback);
	}

	MINIMAL_WINDOWS_API LPVOID WINAPI FlsGetValue(DWORD dwFlsIndex)
	{
		return ::FlsGetValue(dwFlsIndex);
	}

	MINIMAL_WINDOWS_API BOOL WINAPI FlsSetValue(DWORD dwFlsIndex, LPVOID lpFlsValue)
	{
		return ::FlsSetValue(dwFlsIndex, lpFlsValue);
	}

	MINIMAL_WINDOWS_API BOOL WINAPI FlsFree(DWORD dwTlsIndex)
	{
		return ::FlsFree(dwTlsIndex);
	}

	MINIMAL_WINDOWS_API BOOL WINAPI IsProcessorFeaturePresent(DWORD ProcessorFeature)
	{
		return ::IsProcessorFeaturePresent(ProcessorFeature);
	}
}
#endif

