// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AES.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CString.h"
#include "HAL/PlatformMemory.h"

// if x86 AESNI is enabled with compiler, use AESNI without fallback
#if PLATFORM_CPU_X86_FAMILY
#	include <wmmintrin.h>
#	if defined(__AES__) || PLATFORM_ALWAYS_HAS_AESNI
#		define DETECT_HW_AES_SUPPORT_IN_RUNTIME 0
#	else
#		define DETECT_HW_AES_SUPPORT_IN_RUNTIME 1
#	endif
// on ARM enable AES instructions unconditionally only when they are enabled with compiler
#elif PLATFORM_CPU_ARM_FAMILY
#	include <arm_neon.h>
#	ifdef __ARM_FEATURE_AES
#		define DETECT_HW_AES_SUPPORT_IN_RUNTIME 0
#	else
#		define DETECT_HW_AES_SUPPORT_IN_RUNTIME 1
#	endif
#else
// unknown architecture forces SW
#	define FORCE_SW_AES_SUPPORT 1
#endif

#ifndef DETECT_HW_AES_SUPPORT_IN_RUNTIME
#	define DETECT_HW_AES_SUPPORT_IN_RUNTIME 0
#endif
#ifndef FORCE_SW_AES_SUPPORT
#	define FORCE_SW_AES_SUPPORT 0
#endif

#if DETECT_HW_AES_SUPPORT_IN_RUNTIME
#	if PLATFORM_CPU_X86_FAMILY
#		if PLATFORM_COMPILER_CLANG
#			include <cpuid.h>
#		else
#			include <intrin.h> // __cpuid
#		endif
#	elif PLATFORM_ANDROID_ARM64
#		include <cpu-features.h>
#	elif PLATFORM_LINUXARM64
#		include <sys/auxv.h>
#		include <asm/hwcap.h>
#		ifndef HWCAP_AES
#			define HWCAP_AES (1 << 3)
#		endif
#	elif PLATFORM_IOS
#		include "IOS/IOSPlatformMisc.h"
#	elif PLATFORM_WINDOWS // This one handles Windows on ARM64.
#		include "Windows/AllowWindowsPlatformTypes.h"
#		include <Windows.h>
#		include <processthreadsapi.h>
#		include "Windows/HideWindowsPlatformTypes.h"
#	endif
#	if !PLATFORM_WINDOWS
#		include <signal.h>
#		include <setjmp.h>
#	endif
#endif

#define AES256_ROUND_COUNT 14

typedef void AesFunc(const uint8* Key, uint8* Contents, uint64 NumBytes);

static AesFunc AesEncryptX86HW;
static AesFunc AesDecryptX86HW;
static AesFunc AesEncryptArmHW;
static AesFunc AesDecryptArmHW;

static inline uint32 RotateRight(uint32 X, int N)
{
	return (X >> N) | (X << (32 - N));
}

#if !DETECT_HW_AES_SUPPORT_IN_RUNTIME && !FORCE_SW_AES_SUPPORT

#if PLATFORM_CPU_X86_FAMILY
constexpr static AesFunc* AesEncrypt = AesEncryptX86HW;
constexpr static AesFunc* AesDecrypt = AesDecryptX86HW;
#elif PLATFORM_CPU_ARM_FAMILY
constexpr static AesFunc* AesEncrypt = AesEncryptArmHW;
constexpr static AesFunc* AesDecrypt = AesDecryptArmHW;
#else
#error Unknown CPU shouldn't get here
#endif

#else

#if FORCE_SW_AES_SUPPORT

// force software for unknown architecture
static AesFunc AesEncryptSW;
static AesFunc AesDecryptSW;
static AesFunc* AesEncrypt = AesEncryptSW;
static AesFunc* AesDecrypt = AesDecryptSW;

#else

static AesFunc AesEncryptSW;
static AesFunc AesDecryptSW;
static AesFunc AesEncryptRuntimeDispatch;
static AesFunc AesDecryptRuntimeDispatch;

// Those are set to the dispatch functions, so if FAesRuntimeHWSupportDetection
// is gone (or static initialization is not desired), the code still works,
// doing the implementation selection on demand.
static AesFunc* AesEncrypt = &AesEncryptRuntimeDispatch;
static AesFunc* AesDecrypt = &AesDecryptRuntimeDispatch;

#if !PLATFORM_WINDOWS
static sigjmp_buf AesHwJump;
static void AesHwSignalHandler(int Sig)
{
	siglongjmp(AesHwJump, Sig);
}
#endif

static bool HasHardwareSupport()
{
#if PLATFORM_CPU_X86_FAMILY
	int info[4];
#	if PLATFORM_COMPILER_CLANG
	__cpuid(1, info[0], info[1], info[2], info[3]);
#	else
	__cpuid(info, 1);
#	endif
	return (info[2] & 0x02000000) != 0;
#elif PLATFORM_ANDROID_ARM64
	return android_getCpuFeatures() & ANDROID_CPU_ARM64_FEATURE_AES;
#elif PLATFORM_LINUXARM64
	return getauxval(AT_HWCAP) & HWCAP_AES;
#elif PLATFORM_IOS
	return FPlatformMisc::CPUHasHwAesSupport();
#elif PLATFORM_WINDOWS
	return ::IsProcessorFeaturePresent(PF_ARM_V8_CRYPTO_INSTRUCTIONS_AVAILABLE) != 0;
#else
	sigset_t SignalSet;
	sigemptyset(&SignalSet);
	sigaddset(&SignalSet, SIGILL);

	struct sigaction SigAction;
	struct sigaction OldSigAction;
	sigset_t OldSignalSet;
	memset(&SigAction, 0, sizeof(SigAction));
	SigAction.sa_handler = AesHwSignalHandler;
	SigAction.sa_mask = SignalSet;

	sigprocmask(SIG_SETMASK, &SigAction.sa_mask, &OldSignalSet);
	sigaction(SIGILL, &SigAction, &OldSigAction);

	bool bPassed = false;

	if (sigsetjmp(AesHwJump, 1) == 0)
	{
		uint8x16_t Dummy = vaeseq_u8(vdupq_n_u8(0), vdupq_n_u8(0));
		bPassed = true;
	}

	sigaction(SIGILL, &OldSigAction, nullptr);
	sigprocmask(SIG_SETMASK, &OldSignalSet, nullptr);

	return bPassed;
#endif
}

static void PickImplementation()
{
	AesEncrypt = AesEncryptSW;
	AesDecrypt = AesDecryptSW;
	if (HasHardwareSupport())
	{
#if PLATFORM_CPU_X86_FAMILY		
		AesEncrypt = AesEncryptX86HW;
		AesDecrypt = AesDecryptX86HW;
#elif PLATFORM_CPU_ARM_FAMILY
		AesEncrypt = AesEncryptArmHW;
		AesDecrypt = AesDecryptArmHW;
#endif		
	}
}

static void AesEncryptRuntimeDispatch(const uint8* Key, uint8* Contents, uint64 NumBytes)
{
	PickImplementation();
	check(AesEncrypt != &AesEncryptRuntimeDispatch);
	return AesEncrypt(Key, Contents, NumBytes);
}

static void AesDecryptRuntimeDispatch(const uint8* Key, uint8* Contents, uint64 NumBytes)
{
	PickImplementation();
	check(AesDecrypt != &AesDecryptRuntimeDispatch);
	return AesDecrypt(Key, Contents, NumBytes);
}

static struct FAesRuntimeHWSupportDetection
{
	FAesRuntimeHWSupportDetection()
	{
		PickImplementation();
	}
} AesRuntimeHWSupportDetection;

#endif // !FORCE_SW_AES_SUPPORT

struct FAesExpandedKey
{
	uint32 Key[4 * (AES256_ROUND_COUNT + 1)];
};

static const uint8 AesSbox[256] =
{
	0x63, 0x7c, 0x77, 0x7b, 0xf2, 0x6b, 0x6f, 0xc5, 0x30, 0x01, 0x67, 0x2b, 0xfe, 0xd7, 0xab, 0x76,
	0xca, 0x82, 0xc9, 0x7d, 0xfa, 0x59, 0x47, 0xf0, 0xad, 0xd4, 0xa2, 0xaf, 0x9c, 0xa4, 0x72, 0xc0,
	0xb7, 0xfd, 0x93, 0x26, 0x36, 0x3f, 0xf7, 0xcc, 0x34, 0xa5, 0xe5, 0xf1, 0x71, 0xd8, 0x31, 0x15,
	0x04, 0xc7, 0x23, 0xc3, 0x18, 0x96, 0x05, 0x9a, 0x07, 0x12, 0x80, 0xe2, 0xeb, 0x27, 0xb2, 0x75,
	0x09, 0x83, 0x2c, 0x1a, 0x1b, 0x6e, 0x5a, 0xa0, 0x52, 0x3b, 0xd6, 0xb3, 0x29, 0xe3, 0x2f, 0x84,
	0x53, 0xd1, 0x00, 0xed, 0x20, 0xfc, 0xb1, 0x5b, 0x6a, 0xcb, 0xbe, 0x39, 0x4a, 0x4c, 0x58, 0xcf,
	0xd0, 0xef, 0xaa, 0xfb, 0x43, 0x4d, 0x33, 0x85, 0x45, 0xf9, 0x02, 0x7f, 0x50, 0x3c, 0x9f, 0xa8,
	0x51, 0xa3, 0x40, 0x8f, 0x92, 0x9d, 0x38, 0xf5, 0xbc, 0xb6, 0xda, 0x21, 0x10, 0xff, 0xf3, 0xd2,
	0xcd, 0x0c, 0x13, 0xec, 0x5f, 0x97, 0x44, 0x17, 0xc4, 0xa7, 0x7e, 0x3d, 0x64, 0x5d, 0x19, 0x73,
	0x60, 0x81, 0x4f, 0xdc, 0x22, 0x2a, 0x90, 0x88, 0x46, 0xee, 0xb8, 0x14, 0xde, 0x5e, 0x0b, 0xdb,
	0xe0, 0x32, 0x3a, 0x0a, 0x49, 0x06, 0x24, 0x5c, 0xc2, 0xd3, 0xac, 0x62, 0x91, 0x95, 0xe4, 0x79,
	0xe7, 0xc8, 0x37, 0x6d, 0x8d, 0xd5, 0x4e, 0xa9, 0x6c, 0x56, 0xf4, 0xea, 0x65, 0x7a, 0xae, 0x08,
	0xba, 0x78, 0x25, 0x2e, 0x1c, 0xa6, 0xb4, 0xc6, 0xe8, 0xdd, 0x74, 0x1f, 0x4b, 0xbd, 0x8b, 0x8a,
	0x70, 0x3e, 0xb5, 0x66, 0x48, 0x03, 0xf6, 0x0e, 0x61, 0x35, 0x57, 0xb9, 0x86, 0xc1, 0x1d, 0x9e,
	0xe1, 0xf8, 0x98, 0x11, 0x69, 0xd9, 0x8e, 0x94, 0x9b, 0x1e, 0x87, 0xe9, 0xce, 0x55, 0x28, 0xdf,
	0x8c, 0xa1, 0x89, 0x0d, 0xbf, 0xe6, 0x42, 0x68, 0x41, 0x99, 0x2d, 0x0f, 0xb0, 0x54, 0xbb, 0x16,
};

static const uint8 AesISbox[256] =
{
	0x52, 0x09, 0x6a, 0xd5, 0x30, 0x36, 0xa5, 0x38, 0xbf, 0x40, 0xa3, 0x9e, 0x81, 0xf3, 0xd7, 0xfb,
	0x7c, 0xe3, 0x39, 0x82, 0x9b, 0x2f, 0xff, 0x87, 0x34, 0x8e, 0x43, 0x44, 0xc4, 0xde, 0xe9, 0xcb,
	0x54, 0x7b, 0x94, 0x32, 0xa6, 0xc2, 0x23, 0x3d, 0xee, 0x4c, 0x95, 0x0b, 0x42, 0xfa, 0xc3, 0x4e,
	0x08, 0x2e, 0xa1, 0x66, 0x28, 0xd9, 0x24, 0xb2, 0x76, 0x5b, 0xa2, 0x49, 0x6d, 0x8b, 0xd1, 0x25,
	0x72, 0xf8, 0xf6, 0x64, 0x86, 0x68, 0x98, 0x16, 0xd4, 0xa4, 0x5c, 0xcc, 0x5d, 0x65, 0xb6, 0x92,
	0x6c, 0x70, 0x48, 0x50, 0xfd, 0xed, 0xb9, 0xda, 0x5e, 0x15, 0x46, 0x57, 0xa7, 0x8d, 0x9d, 0x84,
	0x90, 0xd8, 0xab, 0x00, 0x8c, 0xbc, 0xd3, 0x0a, 0xf7, 0xe4, 0x58, 0x05, 0xb8, 0xb3, 0x45, 0x06,
	0xd0, 0x2c, 0x1e, 0x8f, 0xca, 0x3f, 0x0f, 0x02, 0xc1, 0xaf, 0xbd, 0x03, 0x01, 0x13, 0x8a, 0x6b,
	0x3a, 0x91, 0x11, 0x41, 0x4f, 0x67, 0xdc, 0xea, 0x97, 0xf2, 0xcf, 0xce, 0xf0, 0xb4, 0xe6, 0x73,
	0x96, 0xac, 0x74, 0x22, 0xe7, 0xad, 0x35, 0x85, 0xe2, 0xf9, 0x37, 0xe8, 0x1c, 0x75, 0xdf, 0x6e,
	0x47, 0xf1, 0x1a, 0x71, 0x1d, 0x29, 0xc5, 0x89, 0x6f, 0xb7, 0x62, 0x0e, 0xaa, 0x18, 0xbe, 0x1b,
	0xfc, 0x56, 0x3e, 0x4b, 0xc6, 0xd2, 0x79, 0x20, 0x9a, 0xdb, 0xc0, 0xfe, 0x78, 0xcd, 0x5a, 0xf4,
	0x1f, 0xdd, 0xa8, 0x33, 0x88, 0x07, 0xc7, 0x31, 0xb1, 0x12, 0x10, 0x59, 0x27, 0x80, 0xec, 0x5f,
	0x60, 0x51, 0x7f, 0xa9, 0x19, 0xb5, 0x4a, 0x0d, 0x2d, 0xe5, 0x7a, 0x9f, 0x93, 0xc9, 0x9c, 0xef,
	0xa0, 0xe0, 0x3b, 0x4d, 0xae, 0x2a, 0xf5, 0xb0, 0xc8, 0xeb, 0xbb, 0x3c, 0x83, 0x53, 0x99, 0x61,
	0x17, 0x2b, 0x04, 0x7e, 0xba, 0x77, 0xd6, 0x26, 0xe1, 0x69, 0x14, 0x63, 0x55, 0x21, 0x0c, 0x7d,
};

static const uint32 AesTbox[4][256] =
{
	{
		0xa56363c6, 0x847c7cf8, 0x997777ee, 0x8d7b7bf6, 0x0df2f2ff, 0xbd6b6bd6, 0xb16f6fde, 0x54c5c591,
		0x50303060, 0x03010102, 0xa96767ce, 0x7d2b2b56, 0x19fefee7, 0x62d7d7b5, 0xe6abab4d, 0x9a7676ec,
		0x45caca8f, 0x9d82821f, 0x40c9c989, 0x877d7dfa, 0x15fafaef, 0xeb5959b2, 0xc947478e, 0x0bf0f0fb,
		0xecadad41, 0x67d4d4b3, 0xfda2a25f, 0xeaafaf45, 0xbf9c9c23, 0xf7a4a453, 0x967272e4, 0x5bc0c09b,
		0xc2b7b775, 0x1cfdfde1, 0xae93933d, 0x6a26264c, 0x5a36366c, 0x413f3f7e, 0x02f7f7f5, 0x4fcccc83,
		0x5c343468, 0xf4a5a551, 0x34e5e5d1, 0x08f1f1f9, 0x937171e2, 0x73d8d8ab, 0x53313162, 0x3f15152a,
		0x0c040408, 0x52c7c795, 0x65232346, 0x5ec3c39d, 0x28181830, 0xa1969637, 0x0f05050a, 0xb59a9a2f,
		0x0907070e, 0x36121224, 0x9b80801b, 0x3de2e2df, 0x26ebebcd, 0x6927274e, 0xcdb2b27f, 0x9f7575ea,
		0x1b090912, 0x9e83831d, 0x742c2c58, 0x2e1a1a34, 0x2d1b1b36, 0xb26e6edc, 0xee5a5ab4, 0xfba0a05b,
		0xf65252a4, 0x4d3b3b76, 0x61d6d6b7, 0xceb3b37d, 0x7b292952, 0x3ee3e3dd, 0x712f2f5e, 0x97848413,
		0xf55353a6, 0x68d1d1b9, 0x00000000, 0x2cededc1, 0x60202040, 0x1ffcfce3, 0xc8b1b179, 0xed5b5bb6,
		0xbe6a6ad4, 0x46cbcb8d, 0xd9bebe67, 0x4b393972, 0xde4a4a94, 0xd44c4c98, 0xe85858b0, 0x4acfcf85,
		0x6bd0d0bb, 0x2aefefc5, 0xe5aaaa4f, 0x16fbfbed, 0xc5434386, 0xd74d4d9a, 0x55333366, 0x94858511,
		0xcf45458a, 0x10f9f9e9, 0x06020204, 0x817f7ffe, 0xf05050a0, 0x443c3c78, 0xba9f9f25, 0xe3a8a84b,
		0xf35151a2, 0xfea3a35d, 0xc0404080, 0x8a8f8f05, 0xad92923f, 0xbc9d9d21, 0x48383870, 0x04f5f5f1,
		0xdfbcbc63, 0xc1b6b677, 0x75dadaaf, 0x63212142, 0x30101020, 0x1affffe5, 0x0ef3f3fd, 0x6dd2d2bf,
		0x4ccdcd81, 0x140c0c18, 0x35131326, 0x2fececc3, 0xe15f5fbe, 0xa2979735, 0xcc444488, 0x3917172e,
		0x57c4c493, 0xf2a7a755, 0x827e7efc, 0x473d3d7a, 0xac6464c8, 0xe75d5dba, 0x2b191932, 0x957373e6,
		0xa06060c0, 0x98818119, 0xd14f4f9e, 0x7fdcdca3, 0x66222244, 0x7e2a2a54, 0xab90903b, 0x8388880b,
		0xca46468c, 0x29eeeec7, 0xd3b8b86b, 0x3c141428, 0x79dedea7, 0xe25e5ebc, 0x1d0b0b16, 0x76dbdbad,
		0x3be0e0db, 0x56323264, 0x4e3a3a74, 0x1e0a0a14, 0xdb494992, 0x0a06060c, 0x6c242448, 0xe45c5cb8,
		0x5dc2c29f, 0x6ed3d3bd, 0xefacac43, 0xa66262c4, 0xa8919139, 0xa4959531, 0x37e4e4d3, 0x8b7979f2,
		0x32e7e7d5, 0x43c8c88b, 0x5937376e, 0xb76d6dda, 0x8c8d8d01, 0x64d5d5b1, 0xd24e4e9c, 0xe0a9a949,
		0xb46c6cd8, 0xfa5656ac, 0x07f4f4f3, 0x25eaeacf, 0xaf6565ca, 0x8e7a7af4, 0xe9aeae47, 0x18080810,
		0xd5baba6f, 0x887878f0, 0x6f25254a, 0x722e2e5c, 0x241c1c38, 0xf1a6a657, 0xc7b4b473, 0x51c6c697,
		0x23e8e8cb, 0x7cdddda1, 0x9c7474e8, 0x211f1f3e, 0xdd4b4b96, 0xdcbdbd61, 0x868b8b0d, 0x858a8a0f,
		0x907070e0, 0x423e3e7c, 0xc4b5b571, 0xaa6666cc, 0xd8484890, 0x05030306, 0x01f6f6f7, 0x120e0e1c,
		0xa36161c2, 0x5f35356a, 0xf95757ae, 0xd0b9b969, 0x91868617, 0x58c1c199, 0x271d1d3a, 0xb99e9e27,
		0x38e1e1d9, 0x13f8f8eb, 0xb398982b, 0x33111122, 0xbb6969d2, 0x70d9d9a9, 0x898e8e07, 0xa7949433,
		0xb69b9b2d, 0x221e1e3c, 0x92878715, 0x20e9e9c9, 0x49cece87, 0xff5555aa, 0x78282850, 0x7adfdfa5,
		0x8f8c8c03, 0xf8a1a159, 0x80898909, 0x170d0d1a, 0xdabfbf65, 0x31e6e6d7, 0xc6424284, 0xb86868d0,
		0xc3414182, 0xb0999929, 0x772d2d5a, 0x110f0f1e, 0xcbb0b07b, 0xfc5454a8, 0xd6bbbb6d, 0x3a16162c,
	},
	{
		0x6363c6a5, 0x7c7cf884, 0x7777ee99, 0x7b7bf68d, 0xf2f2ff0d, 0x6b6bd6bd, 0x6f6fdeb1, 0xc5c59154,
		0x30306050, 0x01010203, 0x6767cea9, 0x2b2b567d, 0xfefee719, 0xd7d7b562, 0xabab4de6, 0x7676ec9a,
		0xcaca8f45, 0x82821f9d, 0xc9c98940, 0x7d7dfa87, 0xfafaef15, 0x5959b2eb, 0x47478ec9, 0xf0f0fb0b,
		0xadad41ec, 0xd4d4b367, 0xa2a25ffd, 0xafaf45ea, 0x9c9c23bf, 0xa4a453f7, 0x7272e496, 0xc0c09b5b,
		0xb7b775c2, 0xfdfde11c, 0x93933dae, 0x26264c6a, 0x36366c5a, 0x3f3f7e41, 0xf7f7f502, 0xcccc834f,
		0x3434685c, 0xa5a551f4, 0xe5e5d134, 0xf1f1f908, 0x7171e293, 0xd8d8ab73, 0x31316253, 0x15152a3f,
		0x0404080c, 0xc7c79552, 0x23234665, 0xc3c39d5e, 0x18183028, 0x969637a1, 0x05050a0f, 0x9a9a2fb5,
		0x07070e09, 0x12122436, 0x80801b9b, 0xe2e2df3d, 0xebebcd26, 0x27274e69, 0xb2b27fcd, 0x7575ea9f,
		0x0909121b, 0x83831d9e, 0x2c2c5874, 0x1a1a342e, 0x1b1b362d, 0x6e6edcb2, 0x5a5ab4ee, 0xa0a05bfb,
		0x5252a4f6, 0x3b3b764d, 0xd6d6b761, 0xb3b37dce, 0x2929527b, 0xe3e3dd3e, 0x2f2f5e71, 0x84841397,
		0x5353a6f5, 0xd1d1b968, 0x00000000, 0xededc12c, 0x20204060, 0xfcfce31f, 0xb1b179c8, 0x5b5bb6ed,
		0x6a6ad4be, 0xcbcb8d46, 0xbebe67d9, 0x3939724b, 0x4a4a94de, 0x4c4c98d4, 0x5858b0e8, 0xcfcf854a,
		0xd0d0bb6b, 0xefefc52a, 0xaaaa4fe5, 0xfbfbed16, 0x434386c5, 0x4d4d9ad7, 0x33336655, 0x85851194,
		0x45458acf, 0xf9f9e910, 0x02020406, 0x7f7ffe81, 0x5050a0f0, 0x3c3c7844, 0x9f9f25ba, 0xa8a84be3,
		0x5151a2f3, 0xa3a35dfe, 0x404080c0, 0x8f8f058a, 0x92923fad, 0x9d9d21bc, 0x38387048, 0xf5f5f104,
		0xbcbc63df, 0xb6b677c1, 0xdadaaf75, 0x21214263, 0x10102030, 0xffffe51a, 0xf3f3fd0e, 0xd2d2bf6d,
		0xcdcd814c, 0x0c0c1814, 0x13132635, 0xececc32f, 0x5f5fbee1, 0x979735a2, 0x444488cc, 0x17172e39,
		0xc4c49357, 0xa7a755f2, 0x7e7efc82, 0x3d3d7a47, 0x6464c8ac, 0x5d5dbae7, 0x1919322b, 0x7373e695,
		0x6060c0a0, 0x81811998, 0x4f4f9ed1, 0xdcdca37f, 0x22224466, 0x2a2a547e, 0x90903bab, 0x88880b83,
		0x46468cca, 0xeeeec729, 0xb8b86bd3, 0x1414283c, 0xdedea779, 0x5e5ebce2, 0x0b0b161d, 0xdbdbad76,
		0xe0e0db3b, 0x32326456, 0x3a3a744e, 0x0a0a141e, 0x494992db, 0x06060c0a, 0x2424486c, 0x5c5cb8e4,
		0xc2c29f5d, 0xd3d3bd6e, 0xacac43ef, 0x6262c4a6, 0x919139a8, 0x959531a4, 0xe4e4d337, 0x7979f28b,
		0xe7e7d532, 0xc8c88b43, 0x37376e59, 0x6d6ddab7, 0x8d8d018c, 0xd5d5b164, 0x4e4e9cd2, 0xa9a949e0,
		0x6c6cd8b4, 0x5656acfa, 0xf4f4f307, 0xeaeacf25, 0x6565caaf, 0x7a7af48e, 0xaeae47e9, 0x08081018,
		0xbaba6fd5, 0x7878f088, 0x25254a6f, 0x2e2e5c72, 0x1c1c3824, 0xa6a657f1, 0xb4b473c7, 0xc6c69751,
		0xe8e8cb23, 0xdddda17c, 0x7474e89c, 0x1f1f3e21, 0x4b4b96dd, 0xbdbd61dc, 0x8b8b0d86, 0x8a8a0f85,
		0x7070e090, 0x3e3e7c42, 0xb5b571c4, 0x6666ccaa, 0x484890d8, 0x03030605, 0xf6f6f701, 0x0e0e1c12,
		0x6161c2a3, 0x35356a5f, 0x5757aef9, 0xb9b969d0, 0x86861791, 0xc1c19958, 0x1d1d3a27, 0x9e9e27b9,
		0xe1e1d938, 0xf8f8eb13, 0x98982bb3, 0x11112233, 0x6969d2bb, 0xd9d9a970, 0x8e8e0789, 0x949433a7,
		0x9b9b2db6, 0x1e1e3c22, 0x87871592, 0xe9e9c920, 0xcece8749, 0x5555aaff, 0x28285078, 0xdfdfa57a,
		0x8c8c038f, 0xa1a159f8, 0x89890980, 0x0d0d1a17, 0xbfbf65da, 0xe6e6d731, 0x424284c6, 0x6868d0b8,
		0x414182c3, 0x999929b0, 0x2d2d5a77, 0x0f0f1e11, 0xb0b07bcb, 0x5454a8fc, 0xbbbb6dd6, 0x16162c3a,
	},
	{
		0x63c6a563, 0x7cf8847c, 0x77ee9977, 0x7bf68d7b, 0xf2ff0df2, 0x6bd6bd6b, 0x6fdeb16f, 0xc59154c5,
		0x30605030, 0x01020301, 0x67cea967, 0x2b567d2b, 0xfee719fe, 0xd7b562d7, 0xab4de6ab, 0x76ec9a76,
		0xca8f45ca, 0x821f9d82, 0xc98940c9, 0x7dfa877d, 0xfaef15fa, 0x59b2eb59, 0x478ec947, 0xf0fb0bf0,
		0xad41ecad, 0xd4b367d4, 0xa25ffda2, 0xaf45eaaf, 0x9c23bf9c, 0xa453f7a4, 0x72e49672, 0xc09b5bc0,
		0xb775c2b7, 0xfde11cfd, 0x933dae93, 0x264c6a26, 0x366c5a36, 0x3f7e413f, 0xf7f502f7, 0xcc834fcc,
		0x34685c34, 0xa551f4a5, 0xe5d134e5, 0xf1f908f1, 0x71e29371, 0xd8ab73d8, 0x31625331, 0x152a3f15,
		0x04080c04, 0xc79552c7, 0x23466523, 0xc39d5ec3, 0x18302818, 0x9637a196, 0x050a0f05, 0x9a2fb59a,
		0x070e0907, 0x12243612, 0x801b9b80, 0xe2df3de2, 0xebcd26eb, 0x274e6927, 0xb27fcdb2, 0x75ea9f75,
		0x09121b09, 0x831d9e83, 0x2c58742c, 0x1a342e1a, 0x1b362d1b, 0x6edcb26e, 0x5ab4ee5a, 0xa05bfba0,
		0x52a4f652, 0x3b764d3b, 0xd6b761d6, 0xb37dceb3, 0x29527b29, 0xe3dd3ee3, 0x2f5e712f, 0x84139784,
		0x53a6f553, 0xd1b968d1, 0x00000000, 0xedc12ced, 0x20406020, 0xfce31ffc, 0xb179c8b1, 0x5bb6ed5b,
		0x6ad4be6a, 0xcb8d46cb, 0xbe67d9be, 0x39724b39, 0x4a94de4a, 0x4c98d44c, 0x58b0e858, 0xcf854acf,
		0xd0bb6bd0, 0xefc52aef, 0xaa4fe5aa, 0xfbed16fb, 0x4386c543, 0x4d9ad74d, 0x33665533, 0x85119485,
		0x458acf45, 0xf9e910f9, 0x02040602, 0x7ffe817f, 0x50a0f050, 0x3c78443c, 0x9f25ba9f, 0xa84be3a8,
		0x51a2f351, 0xa35dfea3, 0x4080c040, 0x8f058a8f, 0x923fad92, 0x9d21bc9d, 0x38704838, 0xf5f104f5,
		0xbc63dfbc, 0xb677c1b6, 0xdaaf75da, 0x21426321, 0x10203010, 0xffe51aff, 0xf3fd0ef3, 0xd2bf6dd2,
		0xcd814ccd, 0x0c18140c, 0x13263513, 0xecc32fec, 0x5fbee15f, 0x9735a297, 0x4488cc44, 0x172e3917,
		0xc49357c4, 0xa755f2a7, 0x7efc827e, 0x3d7a473d, 0x64c8ac64, 0x5dbae75d, 0x19322b19, 0x73e69573,
		0x60c0a060, 0x81199881, 0x4f9ed14f, 0xdca37fdc, 0x22446622, 0x2a547e2a, 0x903bab90, 0x880b8388,
		0x468cca46, 0xeec729ee, 0xb86bd3b8, 0x14283c14, 0xdea779de, 0x5ebce25e, 0x0b161d0b, 0xdbad76db,
		0xe0db3be0, 0x32645632, 0x3a744e3a, 0x0a141e0a, 0x4992db49, 0x060c0a06, 0x24486c24, 0x5cb8e45c,
		0xc29f5dc2, 0xd3bd6ed3, 0xac43efac, 0x62c4a662, 0x9139a891, 0x9531a495, 0xe4d337e4, 0x79f28b79,
		0xe7d532e7, 0xc88b43c8, 0x376e5937, 0x6ddab76d, 0x8d018c8d, 0xd5b164d5, 0x4e9cd24e, 0xa949e0a9,
		0x6cd8b46c, 0x56acfa56, 0xf4f307f4, 0xeacf25ea, 0x65caaf65, 0x7af48e7a, 0xae47e9ae, 0x08101808,
		0xba6fd5ba, 0x78f08878, 0x254a6f25, 0x2e5c722e, 0x1c38241c, 0xa657f1a6, 0xb473c7b4, 0xc69751c6,
		0xe8cb23e8, 0xdda17cdd, 0x74e89c74, 0x1f3e211f, 0x4b96dd4b, 0xbd61dcbd, 0x8b0d868b, 0x8a0f858a,
		0x70e09070, 0x3e7c423e, 0xb571c4b5, 0x66ccaa66, 0x4890d848, 0x03060503, 0xf6f701f6, 0x0e1c120e,
		0x61c2a361, 0x356a5f35, 0x57aef957, 0xb969d0b9, 0x86179186, 0xc19958c1, 0x1d3a271d, 0x9e27b99e,
		0xe1d938e1, 0xf8eb13f8, 0x982bb398, 0x11223311, 0x69d2bb69, 0xd9a970d9, 0x8e07898e, 0x9433a794,
		0x9b2db69b, 0x1e3c221e, 0x87159287, 0xe9c920e9, 0xce8749ce, 0x55aaff55, 0x28507828, 0xdfa57adf,
		0x8c038f8c, 0xa159f8a1, 0x89098089, 0x0d1a170d, 0xbf65dabf, 0xe6d731e6, 0x4284c642, 0x68d0b868,
		0x4182c341, 0x9929b099, 0x2d5a772d, 0x0f1e110f, 0xb07bcbb0, 0x54a8fc54, 0xbb6dd6bb, 0x162c3a16,
	},
	{
		0xc6a56363, 0xf8847c7c, 0xee997777, 0xf68d7b7b, 0xff0df2f2, 0xd6bd6b6b, 0xdeb16f6f, 0x9154c5c5,
		0x60503030, 0x02030101, 0xcea96767, 0x567d2b2b, 0xe719fefe, 0xb562d7d7, 0x4de6abab, 0xec9a7676,
		0x8f45caca, 0x1f9d8282, 0x8940c9c9, 0xfa877d7d, 0xef15fafa, 0xb2eb5959, 0x8ec94747, 0xfb0bf0f0,
		0x41ecadad, 0xb367d4d4, 0x5ffda2a2, 0x45eaafaf, 0x23bf9c9c, 0x53f7a4a4, 0xe4967272, 0x9b5bc0c0,
		0x75c2b7b7, 0xe11cfdfd, 0x3dae9393, 0x4c6a2626, 0x6c5a3636, 0x7e413f3f, 0xf502f7f7, 0x834fcccc,
		0x685c3434, 0x51f4a5a5, 0xd134e5e5, 0xf908f1f1, 0xe2937171, 0xab73d8d8, 0x62533131, 0x2a3f1515,
		0x080c0404, 0x9552c7c7, 0x46652323, 0x9d5ec3c3, 0x30281818, 0x37a19696, 0x0a0f0505, 0x2fb59a9a,
		0x0e090707, 0x24361212, 0x1b9b8080, 0xdf3de2e2, 0xcd26ebeb, 0x4e692727, 0x7fcdb2b2, 0xea9f7575,
		0x121b0909, 0x1d9e8383, 0x58742c2c, 0x342e1a1a, 0x362d1b1b, 0xdcb26e6e, 0xb4ee5a5a, 0x5bfba0a0,
		0xa4f65252, 0x764d3b3b, 0xb761d6d6, 0x7dceb3b3, 0x527b2929, 0xdd3ee3e3, 0x5e712f2f, 0x13978484,
		0xa6f55353, 0xb968d1d1, 0x00000000, 0xc12ceded, 0x40602020, 0xe31ffcfc, 0x79c8b1b1, 0xb6ed5b5b,
		0xd4be6a6a, 0x8d46cbcb, 0x67d9bebe, 0x724b3939, 0x94de4a4a, 0x98d44c4c, 0xb0e85858, 0x854acfcf,
		0xbb6bd0d0, 0xc52aefef, 0x4fe5aaaa, 0xed16fbfb, 0x86c54343, 0x9ad74d4d, 0x66553333, 0x11948585,
		0x8acf4545, 0xe910f9f9, 0x04060202, 0xfe817f7f, 0xa0f05050, 0x78443c3c, 0x25ba9f9f, 0x4be3a8a8,
		0xa2f35151, 0x5dfea3a3, 0x80c04040, 0x058a8f8f, 0x3fad9292, 0x21bc9d9d, 0x70483838, 0xf104f5f5,
		0x63dfbcbc, 0x77c1b6b6, 0xaf75dada, 0x42632121, 0x20301010, 0xe51affff, 0xfd0ef3f3, 0xbf6dd2d2,
		0x814ccdcd, 0x18140c0c, 0x26351313, 0xc32fecec, 0xbee15f5f, 0x35a29797, 0x88cc4444, 0x2e391717,
		0x9357c4c4, 0x55f2a7a7, 0xfc827e7e, 0x7a473d3d, 0xc8ac6464, 0xbae75d5d, 0x322b1919, 0xe6957373,
		0xc0a06060, 0x19988181, 0x9ed14f4f, 0xa37fdcdc, 0x44662222, 0x547e2a2a, 0x3bab9090, 0x0b838888,
		0x8cca4646, 0xc729eeee, 0x6bd3b8b8, 0x283c1414, 0xa779dede, 0xbce25e5e, 0x161d0b0b, 0xad76dbdb,
		0xdb3be0e0, 0x64563232, 0x744e3a3a, 0x141e0a0a, 0x92db4949, 0x0c0a0606, 0x486c2424, 0xb8e45c5c,
		0x9f5dc2c2, 0xbd6ed3d3, 0x43efacac, 0xc4a66262, 0x39a89191, 0x31a49595, 0xd337e4e4, 0xf28b7979,
		0xd532e7e7, 0x8b43c8c8, 0x6e593737, 0xdab76d6d, 0x018c8d8d, 0xb164d5d5, 0x9cd24e4e, 0x49e0a9a9,
		0xd8b46c6c, 0xacfa5656, 0xf307f4f4, 0xcf25eaea, 0xcaaf6565, 0xf48e7a7a, 0x47e9aeae, 0x10180808,
		0x6fd5baba, 0xf0887878, 0x4a6f2525, 0x5c722e2e, 0x38241c1c, 0x57f1a6a6, 0x73c7b4b4, 0x9751c6c6,
		0xcb23e8e8, 0xa17cdddd, 0xe89c7474, 0x3e211f1f, 0x96dd4b4b, 0x61dcbdbd, 0x0d868b8b, 0x0f858a8a,
		0xe0907070, 0x7c423e3e, 0x71c4b5b5, 0xccaa6666, 0x90d84848, 0x06050303, 0xf701f6f6, 0x1c120e0e,
		0xc2a36161, 0x6a5f3535, 0xaef95757, 0x69d0b9b9, 0x17918686, 0x9958c1c1, 0x3a271d1d, 0x27b99e9e,
		0xd938e1e1, 0xeb13f8f8, 0x2bb39898, 0x22331111, 0xd2bb6969, 0xa970d9d9, 0x07898e8e, 0x33a79494,
		0x2db69b9b, 0x3c221e1e, 0x15928787, 0xc920e9e9, 0x8749cece, 0xaaff5555, 0x50782828, 0xa57adfdf,
		0x038f8c8c, 0x59f8a1a1, 0x09808989, 0x1a170d0d, 0x65dabfbf, 0xd731e6e6, 0x84c64242, 0xd0b86868,
		0x82c34141, 0x29b09999, 0x5a772d2d, 0x1e110f0f, 0x7bcbb0b0, 0xa8fc5454, 0x6dd6bbbb, 0x2c3a1616,
	},
};

static const uint32 AesITbox[4][256] =
{
	{
		0x50a7f451, 0x5365417e, 0xc3a4171a, 0x965e273a, 0xcb6bab3b, 0xf1459d1f, 0xab58faac, 0x9303e34b,
		0x55fa3020, 0xf66d76ad, 0x9176cc88, 0x254c02f5, 0xfcd7e54f, 0xd7cb2ac5, 0x80443526, 0x8fa362b5,
		0x495ab1de, 0x671bba25, 0x980eea45, 0xe1c0fe5d, 0x02752fc3, 0x12f04c81, 0xa397468d, 0xc6f9d36b,
		0xe75f8f03, 0x959c9215, 0xeb7a6dbf, 0xda595295, 0x2d83bed4, 0xd3217458, 0x2969e049, 0x44c8c98e,
		0x6a89c275, 0x78798ef4, 0x6b3e5899, 0xdd71b927, 0xb64fe1be, 0x17ad88f0, 0x66ac20c9, 0xb43ace7d,
		0x184adf63, 0x82311ae5, 0x60335197, 0x457f5362, 0xe07764b1, 0x84ae6bbb, 0x1ca081fe, 0x942b08f9,
		0x58684870, 0x19fd458f, 0x876cde94, 0xb7f87b52, 0x23d373ab, 0xe2024b72, 0x578f1fe3, 0x2aab5566,
		0x0728ebb2, 0x03c2b52f, 0x9a7bc586, 0xa50837d3, 0xf2872830, 0xb2a5bf23, 0xba6a0302, 0x5c8216ed,
		0x2b1ccf8a, 0x92b479a7, 0xf0f207f3, 0xa1e2694e, 0xcdf4da65, 0xd5be0506, 0x1f6234d1, 0x8afea6c4,
		0x9d532e34, 0xa055f3a2, 0x32e18a05, 0x75ebf6a4, 0x39ec830b, 0xaaef6040, 0x069f715e, 0x51106ebd,
		0xf98a213e, 0x3d06dd96, 0xae053edd, 0x46bde64d, 0xb58d5491, 0x055dc471, 0x6fd40604, 0xff155060,
		0x24fb9819, 0x97e9bdd6, 0xcc434089, 0x779ed967, 0xbd42e8b0, 0x888b8907, 0x385b19e7, 0xdbeec879,
		0x470a7ca1, 0xe90f427c, 0xc91e84f8, 0x00000000, 0x83868009, 0x48ed2b32, 0xac70111e, 0x4e725a6c,
		0xfbff0efd, 0x5638850f, 0x1ed5ae3d, 0x27392d36, 0x64d90f0a, 0x21a65c68, 0xd1545b9b, 0x3a2e3624,
		0xb1670a0c, 0x0fe75793, 0xd296eeb4, 0x9e919b1b, 0x4fc5c080, 0xa220dc61, 0x694b775a, 0x161a121c,
		0x0aba93e2, 0xe52aa0c0, 0x43e0223c, 0x1d171b12, 0x0b0d090e, 0xadc78bf2, 0xb9a8b62d, 0xc8a91e14,
		0x8519f157, 0x4c0775af, 0xbbdd99ee, 0xfd607fa3, 0x9f2601f7, 0xbcf5725c, 0xc53b6644, 0x347efb5b,
		0x7629438b, 0xdcc623cb, 0x68fcedb6, 0x63f1e4b8, 0xcadc31d7, 0x10856342, 0x40229713, 0x2011c684,
		0x7d244a85, 0xf83dbbd2, 0x1132f9ae, 0x6da129c7, 0x4b2f9e1d, 0xf330b2dc, 0xec52860d, 0xd0e3c177,
		0x6c16b32b, 0x99b970a9, 0xfa489411, 0x2264e947, 0xc48cfca8, 0x1a3ff0a0, 0xd82c7d56, 0xef903322,
		0xc74e4987, 0xc1d138d9, 0xfea2ca8c, 0x360bd498, 0xcf81f5a6, 0x28de7aa5, 0x268eb7da, 0xa4bfad3f,
		0xe49d3a2c, 0x0d927850, 0x9bcc5f6a, 0x62467e54, 0xc2138df6, 0xe8b8d890, 0x5ef7392e, 0xf5afc382,
		0xbe805d9f, 0x7c93d069, 0xa92dd56f, 0xb31225cf, 0x3b99acc8, 0xa77d1810, 0x6e639ce8, 0x7bbb3bdb,
		0x097826cd, 0xf418596e, 0x01b79aec, 0xa89a4f83, 0x656e95e6, 0x7ee6ffaa, 0x08cfbc21, 0xe6e815ef,
		0xd99be7ba, 0xce366f4a, 0xd4099fea, 0xd67cb029, 0xafb2a431, 0x31233f2a, 0x3094a5c6, 0xc066a235,
		0x37bc4e74, 0xa6ca82fc, 0xb0d090e0, 0x15d8a733, 0x4a9804f1, 0xf7daec41, 0x0e50cd7f, 0x2ff69117,
		0x8dd64d76, 0x4db0ef43, 0x544daacc, 0xdf0496e4, 0xe3b5d19e, 0x1b886a4c, 0xb81f2cc1, 0x7f516546,
		0x04ea5e9d, 0x5d358c01, 0x737487fa, 0x2e410bfb, 0x5a1d67b3, 0x52d2db92, 0x335610e9, 0x1347d66d,
		0x8c61d79a, 0x7a0ca137, 0x8e14f859, 0x893c13eb, 0xee27a9ce, 0x35c961b7, 0xede51ce1, 0x3cb1477a,
		0x59dfd29c, 0x3f73f255, 0x79ce1418, 0xbf37c773, 0xeacdf753, 0x5baafd5f, 0x146f3ddf, 0x86db4478,
		0x81f3afca, 0x3ec468b9, 0x2c342438, 0x5f40a3c2, 0x72c31d16, 0x0c25e2bc, 0x8b493c28, 0x41950dff,
		0x7101a839, 0xdeb30c08, 0x9ce4b4d8, 0x90c15664, 0x6184cb7b, 0x70b632d5, 0x745c6c48, 0x4257b8d0,
	},
	{
		0xa7f45150, 0x65417e53, 0xa4171ac3, 0x5e273a96, 0x6bab3bcb, 0x459d1ff1, 0x58faacab, 0x03e34b93,
		0xfa302055, 0x6d76adf6, 0x76cc8891, 0x4c02f525, 0xd7e54ffc, 0xcb2ac5d7, 0x44352680, 0xa362b58f,
		0x5ab1de49, 0x1bba2567, 0x0eea4598, 0xc0fe5de1, 0x752fc302, 0xf04c8112, 0x97468da3, 0xf9d36bc6,
		0x5f8f03e7, 0x9c921595, 0x7a6dbfeb, 0x595295da, 0x83bed42d, 0x217458d3, 0x69e04929, 0xc8c98e44,
		0x89c2756a, 0x798ef478, 0x3e58996b, 0x71b927dd, 0x4fe1beb6, 0xad88f017, 0xac20c966, 0x3ace7db4,
		0x4adf6318, 0x311ae582, 0x33519760, 0x7f536245, 0x7764b1e0, 0xae6bbb84, 0xa081fe1c, 0x2b08f994,
		0x68487058, 0xfd458f19, 0x6cde9487, 0xf87b52b7, 0xd373ab23, 0x024b72e2, 0x8f1fe357, 0xab55662a,
		0x28ebb207, 0xc2b52f03, 0x7bc5869a, 0x0837d3a5, 0x872830f2, 0xa5bf23b2, 0x6a0302ba, 0x8216ed5c,
		0x1ccf8a2b, 0xb479a792, 0xf207f3f0, 0xe2694ea1, 0xf4da65cd, 0xbe0506d5, 0x6234d11f, 0xfea6c48a,
		0x532e349d, 0x55f3a2a0, 0xe18a0532, 0xebf6a475, 0xec830b39, 0xef6040aa, 0x9f715e06, 0x106ebd51,
		0x8a213ef9, 0x06dd963d, 0x053eddae, 0xbde64d46, 0x8d5491b5, 0x5dc47105, 0xd406046f, 0x155060ff,
		0xfb981924, 0xe9bdd697, 0x434089cc, 0x9ed96777, 0x42e8b0bd, 0x8b890788, 0x5b19e738, 0xeec879db,
		0x0a7ca147, 0x0f427ce9, 0x1e84f8c9, 0x00000000, 0x86800983, 0xed2b3248, 0x70111eac, 0x725a6c4e,
		0xff0efdfb, 0x38850f56, 0xd5ae3d1e, 0x392d3627, 0xd90f0a64, 0xa65c6821, 0x545b9bd1, 0x2e36243a,
		0x670a0cb1, 0xe757930f, 0x96eeb4d2, 0x919b1b9e, 0xc5c0804f, 0x20dc61a2, 0x4b775a69, 0x1a121c16,
		0xba93e20a, 0x2aa0c0e5, 0xe0223c43, 0x171b121d, 0x0d090e0b, 0xc78bf2ad, 0xa8b62db9, 0xa91e14c8,
		0x19f15785, 0x0775af4c, 0xdd99eebb, 0x607fa3fd, 0x2601f79f, 0xf5725cbc, 0x3b6644c5, 0x7efb5b34,
		0x29438b76, 0xc623cbdc, 0xfcedb668, 0xf1e4b863, 0xdc31d7ca, 0x85634210, 0x22971340, 0x11c68420,
		0x244a857d, 0x3dbbd2f8, 0x32f9ae11, 0xa129c76d, 0x2f9e1d4b, 0x30b2dcf3, 0x52860dec, 0xe3c177d0,
		0x16b32b6c, 0xb970a999, 0x489411fa, 0x64e94722, 0x8cfca8c4, 0x3ff0a01a, 0x2c7d56d8, 0x903322ef,
		0x4e4987c7, 0xd138d9c1, 0xa2ca8cfe, 0x0bd49836, 0x81f5a6cf, 0xde7aa528, 0x8eb7da26, 0xbfad3fa4,
		0x9d3a2ce4, 0x9278500d, 0xcc5f6a9b, 0x467e5462, 0x138df6c2, 0xb8d890e8, 0xf7392e5e, 0xafc382f5,
		0x805d9fbe, 0x93d0697c, 0x2dd56fa9, 0x1225cfb3, 0x99acc83b, 0x7d1810a7, 0x639ce86e, 0xbb3bdb7b,
		0x7826cd09, 0x18596ef4, 0xb79aec01, 0x9a4f83a8, 0x6e95e665, 0xe6ffaa7e, 0xcfbc2108, 0xe815efe6,
		0x9be7bad9, 0x366f4ace, 0x099fead4, 0x7cb029d6, 0xb2a431af, 0x233f2a31, 0x94a5c630, 0x66a235c0,
		0xbc4e7437, 0xca82fca6, 0xd090e0b0, 0xd8a73315, 0x9804f14a, 0xdaec41f7, 0x50cd7f0e, 0xf691172f,
		0xd64d768d, 0xb0ef434d, 0x4daacc54, 0x0496e4df, 0xb5d19ee3, 0x886a4c1b, 0x1f2cc1b8, 0x5165467f,
		0xea5e9d04, 0x358c015d, 0x7487fa73, 0x410bfb2e, 0x1d67b35a, 0xd2db9252, 0x5610e933, 0x47d66d13,
		0x61d79a8c, 0x0ca1377a, 0x14f8598e, 0x3c13eb89, 0x27a9ceee, 0xc961b735, 0xe51ce1ed, 0xb1477a3c,
		0xdfd29c59, 0x73f2553f, 0xce141879, 0x37c773bf, 0xcdf753ea, 0xaafd5f5b, 0x6f3ddf14, 0xdb447886,
		0xf3afca81, 0xc468b93e, 0x3424382c, 0x40a3c25f, 0xc31d1672, 0x25e2bc0c, 0x493c288b, 0x950dff41,
		0x01a83971, 0xb30c08de, 0xe4b4d89c, 0xc1566490, 0x84cb7b61, 0xb632d570, 0x5c6c4874, 0x57b8d042,
	},
	{
		0xf45150a7, 0x417e5365, 0x171ac3a4, 0x273a965e, 0xab3bcb6b, 0x9d1ff145, 0xfaacab58, 0xe34b9303,
		0x302055fa, 0x76adf66d, 0xcc889176, 0x02f5254c, 0xe54ffcd7, 0x2ac5d7cb, 0x35268044, 0x62b58fa3,
		0xb1de495a, 0xba25671b, 0xea45980e, 0xfe5de1c0, 0x2fc30275, 0x4c8112f0, 0x468da397, 0xd36bc6f9,
		0x8f03e75f, 0x9215959c, 0x6dbfeb7a, 0x5295da59, 0xbed42d83, 0x7458d321, 0xe0492969, 0xc98e44c8,
		0xc2756a89, 0x8ef47879, 0x58996b3e, 0xb927dd71, 0xe1beb64f, 0x88f017ad, 0x20c966ac, 0xce7db43a,
		0xdf63184a, 0x1ae58231, 0x51976033, 0x5362457f, 0x64b1e077, 0x6bbb84ae, 0x81fe1ca0, 0x08f9942b,
		0x48705868, 0x458f19fd, 0xde94876c, 0x7b52b7f8, 0x73ab23d3, 0x4b72e202, 0x1fe3578f, 0x55662aab,
		0xebb20728, 0xb52f03c2, 0xc5869a7b, 0x37d3a508, 0x2830f287, 0xbf23b2a5, 0x0302ba6a, 0x16ed5c82,
		0xcf8a2b1c, 0x79a792b4, 0x07f3f0f2, 0x694ea1e2, 0xda65cdf4, 0x0506d5be, 0x34d11f62, 0xa6c48afe,
		0x2e349d53, 0xf3a2a055, 0x8a0532e1, 0xf6a475eb, 0x830b39ec, 0x6040aaef, 0x715e069f, 0x6ebd5110,
		0x213ef98a, 0xdd963d06, 0x3eddae05, 0xe64d46bd, 0x5491b58d, 0xc471055d, 0x06046fd4, 0x5060ff15,
		0x981924fb, 0xbdd697e9, 0x4089cc43, 0xd967779e, 0xe8b0bd42, 0x8907888b, 0x19e7385b, 0xc879dbee,
		0x7ca1470a, 0x427ce90f, 0x84f8c91e, 0x00000000, 0x80098386, 0x2b3248ed, 0x111eac70, 0x5a6c4e72,
		0x0efdfbff, 0x850f5638, 0xae3d1ed5, 0x2d362739, 0x0f0a64d9, 0x5c6821a6, 0x5b9bd154, 0x36243a2e,
		0x0a0cb167, 0x57930fe7, 0xeeb4d296, 0x9b1b9e91, 0xc0804fc5, 0xdc61a220, 0x775a694b, 0x121c161a,
		0x93e20aba, 0xa0c0e52a, 0x223c43e0, 0x1b121d17, 0x090e0b0d, 0x8bf2adc7, 0xb62db9a8, 0x1e14c8a9,
		0xf1578519, 0x75af4c07, 0x99eebbdd, 0x7fa3fd60, 0x01f79f26, 0x725cbcf5, 0x6644c53b, 0xfb5b347e,
		0x438b7629, 0x23cbdcc6, 0xedb668fc, 0xe4b863f1, 0x31d7cadc, 0x63421085, 0x97134022, 0xc6842011,
		0x4a857d24, 0xbbd2f83d, 0xf9ae1132, 0x29c76da1, 0x9e1d4b2f, 0xb2dcf330, 0x860dec52, 0xc177d0e3,
		0xb32b6c16, 0x70a999b9, 0x9411fa48, 0xe9472264, 0xfca8c48c, 0xf0a01a3f, 0x7d56d82c, 0x3322ef90,
		0x4987c74e, 0x38d9c1d1, 0xca8cfea2, 0xd498360b, 0xf5a6cf81, 0x7aa528de, 0xb7da268e, 0xad3fa4bf,
		0x3a2ce49d, 0x78500d92, 0x5f6a9bcc, 0x7e546246, 0x8df6c213, 0xd890e8b8, 0x392e5ef7, 0xc382f5af,
		0x5d9fbe80, 0xd0697c93, 0xd56fa92d, 0x25cfb312, 0xacc83b99, 0x1810a77d, 0x9ce86e63, 0x3bdb7bbb,
		0x26cd0978, 0x596ef418, 0x9aec01b7, 0x4f83a89a, 0x95e6656e, 0xffaa7ee6, 0xbc2108cf, 0x15efe6e8,
		0xe7bad99b, 0x6f4ace36, 0x9fead409, 0xb029d67c, 0xa431afb2, 0x3f2a3123, 0xa5c63094, 0xa235c066,
		0x4e7437bc, 0x82fca6ca, 0x90e0b0d0, 0xa73315d8, 0x04f14a98, 0xec41f7da, 0xcd7f0e50, 0x91172ff6,
		0x4d768dd6, 0xef434db0, 0xaacc544d, 0x96e4df04, 0xd19ee3b5, 0x6a4c1b88, 0x2cc1b81f, 0x65467f51,
		0x5e9d04ea, 0x8c015d35, 0x87fa7374, 0x0bfb2e41, 0x67b35a1d, 0xdb9252d2, 0x10e93356, 0xd66d1347,
		0xd79a8c61, 0xa1377a0c, 0xf8598e14, 0x13eb893c, 0xa9ceee27, 0x61b735c9, 0x1ce1ede5, 0x477a3cb1,
		0xd29c59df, 0xf2553f73, 0x141879ce, 0xc773bf37, 0xf753eacd, 0xfd5f5baa, 0x3ddf146f, 0x447886db,
		0xafca81f3, 0x68b93ec4, 0x24382c34, 0xa3c25f40, 0x1d1672c3, 0xe2bc0c25, 0x3c288b49, 0x0dff4195,
		0xa8397101, 0x0c08deb3, 0xb4d89ce4, 0x566490c1, 0xcb7b6184, 0x32d570b6, 0x6c48745c, 0xb8d04257,
	},
	{
		0x5150a7f4, 0x7e536541, 0x1ac3a417, 0x3a965e27, 0x3bcb6bab, 0x1ff1459d, 0xacab58fa, 0x4b9303e3,
		0x2055fa30, 0xadf66d76, 0x889176cc, 0xf5254c02, 0x4ffcd7e5, 0xc5d7cb2a, 0x26804435, 0xb58fa362,
		0xde495ab1, 0x25671bba, 0x45980eea, 0x5de1c0fe, 0xc302752f, 0x8112f04c, 0x8da39746, 0x6bc6f9d3,
		0x03e75f8f, 0x15959c92, 0xbfeb7a6d, 0x95da5952, 0xd42d83be, 0x58d32174, 0x492969e0, 0x8e44c8c9,
		0x756a89c2, 0xf478798e, 0x996b3e58, 0x27dd71b9, 0xbeb64fe1, 0xf017ad88, 0xc966ac20, 0x7db43ace,
		0x63184adf, 0xe582311a, 0x97603351, 0x62457f53, 0xb1e07764, 0xbb84ae6b, 0xfe1ca081, 0xf9942b08,
		0x70586848, 0x8f19fd45, 0x94876cde, 0x52b7f87b, 0xab23d373, 0x72e2024b, 0xe3578f1f, 0x662aab55,
		0xb20728eb, 0x2f03c2b5, 0x869a7bc5, 0xd3a50837, 0x30f28728, 0x23b2a5bf, 0x02ba6a03, 0xed5c8216,
		0x8a2b1ccf, 0xa792b479, 0xf3f0f207, 0x4ea1e269, 0x65cdf4da, 0x06d5be05, 0xd11f6234, 0xc48afea6,
		0x349d532e, 0xa2a055f3, 0x0532e18a, 0xa475ebf6, 0x0b39ec83, 0x40aaef60, 0x5e069f71, 0xbd51106e,
		0x3ef98a21, 0x963d06dd, 0xddae053e, 0x4d46bde6, 0x91b58d54, 0x71055dc4, 0x046fd406, 0x60ff1550,
		0x1924fb98, 0xd697e9bd, 0x89cc4340, 0x67779ed9, 0xb0bd42e8, 0x07888b89, 0xe7385b19, 0x79dbeec8,
		0xa1470a7c, 0x7ce90f42, 0xf8c91e84, 0x00000000, 0x09838680, 0x3248ed2b, 0x1eac7011, 0x6c4e725a,
		0xfdfbff0e, 0x0f563885, 0x3d1ed5ae, 0x3627392d, 0x0a64d90f, 0x6821a65c, 0x9bd1545b, 0x243a2e36,
		0x0cb1670a, 0x930fe757, 0xb4d296ee, 0x1b9e919b, 0x804fc5c0, 0x61a220dc, 0x5a694b77, 0x1c161a12,
		0xe20aba93, 0xc0e52aa0, 0x3c43e022, 0x121d171b, 0x0e0b0d09, 0xf2adc78b, 0x2db9a8b6, 0x14c8a91e,
		0x578519f1, 0xaf4c0775, 0xeebbdd99, 0xa3fd607f, 0xf79f2601, 0x5cbcf572, 0x44c53b66, 0x5b347efb,
		0x8b762943, 0xcbdcc623, 0xb668fced, 0xb863f1e4, 0xd7cadc31, 0x42108563, 0x13402297, 0x842011c6,
		0x857d244a, 0xd2f83dbb, 0xae1132f9, 0xc76da129, 0x1d4b2f9e, 0xdcf330b2, 0x0dec5286, 0x77d0e3c1,
		0x2b6c16b3, 0xa999b970, 0x11fa4894, 0x472264e9, 0xa8c48cfc, 0xa01a3ff0, 0x56d82c7d, 0x22ef9033,
		0x87c74e49, 0xd9c1d138, 0x8cfea2ca, 0x98360bd4, 0xa6cf81f5, 0xa528de7a, 0xda268eb7, 0x3fa4bfad,
		0x2ce49d3a, 0x500d9278, 0x6a9bcc5f, 0x5462467e, 0xf6c2138d, 0x90e8b8d8, 0x2e5ef739, 0x82f5afc3,
		0x9fbe805d, 0x697c93d0, 0x6fa92dd5, 0xcfb31225, 0xc83b99ac, 0x10a77d18, 0xe86e639c, 0xdb7bbb3b,
		0xcd097826, 0x6ef41859, 0xec01b79a, 0x83a89a4f, 0xe6656e95, 0xaa7ee6ff, 0x2108cfbc, 0xefe6e815,
		0xbad99be7, 0x4ace366f, 0xead4099f, 0x29d67cb0, 0x31afb2a4, 0x2a31233f, 0xc63094a5, 0x35c066a2,
		0x7437bc4e, 0xfca6ca82, 0xe0b0d090, 0x3315d8a7, 0xf14a9804, 0x41f7daec, 0x7f0e50cd, 0x172ff691,
		0x768dd64d, 0x434db0ef, 0xcc544daa, 0xe4df0496, 0x9ee3b5d1, 0x4c1b886a, 0xc1b81f2c, 0x467f5165,
		0x9d04ea5e, 0x015d358c, 0xfa737487, 0xfb2e410b, 0xb35a1d67, 0x9252d2db, 0xe9335610, 0x6d1347d6,
		0x9a8c61d7, 0x377a0ca1, 0x598e14f8, 0xeb893c13, 0xceee27a9, 0xb735c961, 0xe1ede51c, 0x7a3cb147,
		0x9c59dfd2, 0x553f73f2, 0x1879ce14, 0x73bf37c7, 0x53eacdf7, 0x5f5baafd, 0xdf146f3d, 0x7886db44,
		0xca81f3af, 0xb93ec468, 0x382c3424, 0xc25f40a3, 0x1672c31d, 0xbc0c25e2, 0x288b493c, 0xff41950d,
		0x397101a8, 0x08deb30c, 0xd89ce4b4, 0x6490c156, 0x7b6184cb, 0xd570b632, 0x48745c6c, 0xd04257b8,
	},
};

static inline uint32 AesEncryptMix(uint32 x)
{
	return ((uint32)AesSbox[(uint8)(x >>  0)] <<  0)
		 ^ ((uint32)AesSbox[(uint8)(x >>  8)] <<  8)
		 ^ ((uint32)AesSbox[(uint8)(x >> 16)] << 16)
		 ^ ((uint32)AesSbox[(uint8)(x >> 24)] << 24);
}

static inline uint32 AesDecryptMix(uint32 x)
{
	return (AesITbox[0][AesSbox[(uint8)(x >>  0)]])
		 ^ (AesITbox[1][AesSbox[(uint8)(x >>  8)]])
		 ^ (AesITbox[2][AesSbox[(uint8)(x >> 16)]])
		 ^ (AesITbox[3][AesSbox[(uint8)(x >> 24)]]);
}

static inline void AesEncryptExpand(FAesExpandedKey* EncryptKey, const uint8* Key)
{
	uint32* EKey = EncryptKey->Key;

	EKey[0] = FPlatformMemory::ReadUnaligned<uint32>(Key +  0);
	EKey[1] = FPlatformMemory::ReadUnaligned<uint32>(Key +  4);
	EKey[2] = FPlatformMemory::ReadUnaligned<uint32>(Key +  8);
	EKey[3] = FPlatformMemory::ReadUnaligned<uint32>(Key + 12);
	EKey[4] = FPlatformMemory::ReadUnaligned<uint32>(Key + 16);
	EKey[5] = FPlatformMemory::ReadUnaligned<uint32>(Key + 20);
	EKey[6] = FPlatformMemory::ReadUnaligned<uint32>(Key + 24);
	EKey[7] = FPlatformMemory::ReadUnaligned<uint32>(Key + 28);

	for (int Index = 0; Index < AES256_ROUND_COUNT / 2; Index++, EKey += 8)
	{
		// rcon[] = { 0x01, 0x02, 0x04, 0x08, 0x10, 0x20, 0x40, 0x80 }
		uint32 Rcon = 1U << Index;

		EKey[ 8] = EKey[0] ^ RotateRight(AesEncryptMix(EKey[7]), 8) ^ Rcon;
		EKey[ 9] = EKey[1] ^ EKey[ 8];
		EKey[10] = EKey[2] ^ EKey[ 9];
		EKey[11] = EKey[3] ^ EKey[10];

		if (Index < AES256_ROUND_COUNT / 2 - 1)
		{
			EKey[12] = EKey[4] ^ AesEncryptMix(EKey[11]);
			EKey[13] = EKey[5] ^ EKey[12];
			EKey[14] = EKey[6] ^ EKey[13];
			EKey[15] = EKey[7] ^ EKey[14];
		}
	}
}

static inline void AesDecryptExpand(FAesExpandedKey* DecryptKey, const uint8* Key)
{
	FAesExpandedKey EncryptKey;
	AesEncryptExpand(&EncryptKey, Key);

	const uint32* EKey = EncryptKey.Key + (4 * AES256_ROUND_COUNT);
	uint32* DKey = DecryptKey->Key;

	DKey[0] = EKey[0];
	DKey[1] = EKey[1];
	DKey[2] = EKey[2];
	DKey[3] = EKey[3];
	DKey += 4;
	EKey -= 4;

	for (int Round = 1; Round < AES256_ROUND_COUNT; Round++)
	{
		DKey[0] = AesDecryptMix(EKey[0]);
		DKey[1] = AesDecryptMix(EKey[1]);
		DKey[2] = AesDecryptMix(EKey[2]);
		DKey[3] = AesDecryptMix(EKey[3]);
		DKey += 4;
		EKey -= 4;
	}

	DKey[0] = EKey[0];
	DKey[1] = EKey[1];
	DKey[2] = EKey[2];
	DKey[3] = EKey[3];
}

static inline void AesEncryptSW(const uint8* Key, uint8* Contents, uint64 NumBytes)
{
#define AES_ENC(a, b, c, d) (      \
	AesTbox[0][(uint8)(a >>  0)] ^ \
	AesTbox[1][(uint8)(b >>  8)] ^ \
	AesTbox[2][(uint8)(c >> 16)] ^ \
	AesTbox[3][(uint8)(d >> 24)]   \
)

#define AES_ENC_LAST(a, b, c, d) (      \
	(AesSbox[(uint8)(a >>  0)] <<  0) ^ \
	(AesSbox[(uint8)(b >>  8)] <<  8) ^ \
	(AesSbox[(uint8)(c >> 16)] << 16) ^ \
	(AesSbox[(uint8)(d >> 24)] << 24)   \
)

	FAesExpandedKey EncryptKey;
	AesEncryptExpand(&EncryptKey, Key);

	const uint32* EKey = EncryptKey.Key;

	for (uint64 Offset = 0; Offset < NumBytes; Offset += 16)
	{
		uint8* Block = Contents + Offset;

		uint32 s0 = FPlatformMemory::ReadUnaligned<uint32>(Block +  0) ^ EKey[0];
		uint32 s1 = FPlatformMemory::ReadUnaligned<uint32>(Block +  4) ^ EKey[1];
		uint32 s2 = FPlatformMemory::ReadUnaligned<uint32>(Block +  8) ^ EKey[2];
		uint32 s3 = FPlatformMemory::ReadUnaligned<uint32>(Block + 12) ^ EKey[3];

		for (int Round = 1; Round < AES256_ROUND_COUNT; Round++)
		{
			uint32 t0 = AES_ENC(s0, s1, s2, s3);
			uint32 t1 = AES_ENC(s1, s2, s3, s0);
			uint32 t2 = AES_ENC(s2, s3, s0, s1);
			uint32 t3 = AES_ENC(s3, s0, s1, s2);

			s0 = t0 ^ EKey[4 * Round + 0];
			s1 = t1 ^ EKey[4 * Round + 1];
			s2 = t2 ^ EKey[4 * Round + 2];
			s3 = t3 ^ EKey[4 * Round + 3];
		}

		uint32 t0 = AES_ENC_LAST(s0, s1, s2, s3);
		uint32 t1 = AES_ENC_LAST(s1, s2, s3, s0);
		uint32 t2 = AES_ENC_LAST(s2, s3, s0, s1);
		uint32 t3 = AES_ENC_LAST(s3, s0, s1, s2);

		s0 = t0 ^ EKey[4 * AES256_ROUND_COUNT + 0];
		s1 = t1 ^ EKey[4 * AES256_ROUND_COUNT + 1];
		s2 = t2 ^ EKey[4 * AES256_ROUND_COUNT + 2];
		s3 = t3 ^ EKey[4 * AES256_ROUND_COUNT + 3];

		FPlatformMemory::WriteUnaligned<uint32>(Block +  0, s0);
		FPlatformMemory::WriteUnaligned<uint32>(Block +  4, s1);
		FPlatformMemory::WriteUnaligned<uint32>(Block +  8, s2);
		FPlatformMemory::WriteUnaligned<uint32>(Block + 12, s3);
	}

#undef AES_ENC
#undef AES_ENC_LAST
}

static inline void AesDecryptSW(const uint8* Key, uint8* Contents, uint64 NumBytes)
{
#define AES_DEC(a, b, c, d) (       \
	AesITbox[0][(uint8)(a >>  0)] ^ \
	AesITbox[1][(uint8)(b >>  8)] ^ \
	AesITbox[2][(uint8)(c >> 16)] ^ \
	AesITbox[3][(uint8)(d >> 24)]   \
)

#define AES_DEC_LAST(a, b, c, d) (       \
	(AesISbox[(uint8)(a >>  0)] <<  0) ^ \
	(AesISbox[(uint8)(b >>  8)] <<  8) ^ \
	(AesISbox[(uint8)(c >> 16)] << 16) ^ \
	(AesISbox[(uint8)(d >> 24)] << 24)   \
)

	FAesExpandedKey DecryptKey;
	AesDecryptExpand(&DecryptKey, Key);

	const uint32* DKey = DecryptKey.Key;

	for (uint64 Offset = 0; Offset < NumBytes; Offset += 16)
	{
		uint8* Block = Contents + Offset;

		uint32 s0 = FPlatformMemory::ReadUnaligned<uint32>(Block +  0) ^ DKey[0];
		uint32 s1 = FPlatformMemory::ReadUnaligned<uint32>(Block +  4) ^ DKey[1];
		uint32 s2 = FPlatformMemory::ReadUnaligned<uint32>(Block +  8) ^ DKey[2];
		uint32 s3 = FPlatformMemory::ReadUnaligned<uint32>(Block + 12) ^ DKey[3];

		for (int Round = 1; Round < AES256_ROUND_COUNT; Round++)
		{
			uint32 t0 = AES_DEC(s0, s3, s2, s1);
			uint32 t1 = AES_DEC(s1, s0, s3, s2);
			uint32 t2 = AES_DEC(s2, s1, s0, s3);
			uint32 t3 = AES_DEC(s3, s2, s1, s0);

			s0 = t0 ^ DKey[4 * Round + 0];
			s1 = t1 ^ DKey[4 * Round + 1];
			s2 = t2 ^ DKey[4 * Round + 2];
			s3 = t3 ^ DKey[4 * Round + 3];
		}

		uint32 t0 = AES_DEC_LAST(s0, s3, s2, s1);
		uint32 t1 = AES_DEC_LAST(s1, s0, s3, s2);
		uint32 t2 = AES_DEC_LAST(s2, s1, s0, s3);
		uint32 t3 = AES_DEC_LAST(s3, s2, s1, s0);

		s0 = t0 ^ DKey[4 * AES256_ROUND_COUNT + 0];
		s1 = t1 ^ DKey[4 * AES256_ROUND_COUNT + 1];
		s2 = t2 ^ DKey[4 * AES256_ROUND_COUNT + 2];
		s3 = t3 ^ DKey[4 * AES256_ROUND_COUNT + 3];

		FPlatformMemory::WriteUnaligned<uint32>(Block +  0, s0);
		FPlatformMemory::WriteUnaligned<uint32>(Block +  4, s1);
		FPlatformMemory::WriteUnaligned<uint32>(Block +  8, s2);
		FPlatformMemory::WriteUnaligned<uint32>(Block + 12, s3);
	}

#undef AES_DEC
#undef AES_DEC_LAST
}


#endif // DETECT_HW_AES_SUPPORT_IN_RUNTIME

#if PLATFORM_CPU_X86_FAMILY

// https://www.intel.com/content/dam/doc/white-paper/advanced-encryption-standard-new-instructions-set-paper.pdf

#define AES_INIT_ENC_X86(ctx, x, y, rcon)         \
{                                                 \
    __m128i z, t;                                 \
    ctx = y;                                      \
    z = _mm_aeskeygenassist_si128(y, rcon);       \
    z = _mm_shuffle_epi32(z, rcon ? 0xff : 0xaa); \
    t = _mm_slli_si128(x, 4);                     \
    x = _mm_xor_si128(x, t);                      \
    t = _mm_slli_si128(t, 4);                     \
    x = _mm_xor_si128(x, t);                      \
    t = _mm_slli_si128(t, 4);                     \
    x = _mm_xor_si128(x, t);                      \
    x = _mm_xor_si128(x, z);                      \
}

#define AES_INIT_DEC_X86(ctx, x, y, rcon)         \
{                                                 \
    __m128i z, t;                                 \
    ctx = _mm_aesimc_si128(y);                    \
    z = _mm_aeskeygenassist_si128(y, rcon);       \
    z = _mm_shuffle_epi32(z, rcon ? 0xff : 0xaa); \
    t = _mm_slli_si128(x, 4);                     \
    x = _mm_xor_si128(x, t);                      \
    t = _mm_slli_si128(t, 4);                     \
    x = _mm_xor_si128(x, t);                      \
    t = _mm_slli_si128(t, 4);                     \
    x = _mm_xor_si128(x, t);                      \
    x = _mm_xor_si128(x, z);                      \
}

#if PLATFORM_COMPILER_CLANG
#pragma clang attribute push (__attribute__((target("aes"))), apply_to=function)
#endif

static inline void AesEncryptX86HW(const uint8* Key, uint8* Contents, uint64 NumBytes)
{
	const __m128i* Key128 = reinterpret_cast<const __m128i*>(Key);
	__m128i x = _mm_loadu_si128(Key128 + 0);
	__m128i y = _mm_loadu_si128(Key128 + 1);

	__m128i EKey[AES256_ROUND_COUNT + 1];
	EKey[0] = x;
	AES_INIT_ENC_X86(EKey[ 1], x, y, 0x01);
	AES_INIT_ENC_X86(EKey[ 2], y, x, 0x00);
	AES_INIT_ENC_X86(EKey[ 3], x, y, 0x02);
	AES_INIT_ENC_X86(EKey[ 4], y, x, 0x00);
	AES_INIT_ENC_X86(EKey[ 5], x, y, 0x04);
	AES_INIT_ENC_X86(EKey[ 6], y, x, 0x00);
	AES_INIT_ENC_X86(EKey[ 7], x, y, 0x08);
	AES_INIT_ENC_X86(EKey[ 8], y, x, 0x00);
	AES_INIT_ENC_X86(EKey[ 9], x, y, 0x10);
	AES_INIT_ENC_X86(EKey[10], y, x, 0x00);
	AES_INIT_ENC_X86(EKey[11], x, y, 0x20);
	AES_INIT_ENC_X86(EKey[12], y, x, 0x00);
	AES_INIT_ENC_X86(EKey[13], x, y, 0x40);
	EKey[14] = x;

	// unrolling loop to do multiple AES blocks is faster on newer CPUs
	// for example, ~10% improvement on Zen2 Threadripper (4989 -> 5501 MB/s)
	__m128i* Data = reinterpret_cast<__m128i*>(Contents);
	while (NumBytes >= 32)
	{
		__m128i Block0 = _mm_loadu_si128(Data + 0);
		__m128i Block1 = _mm_loadu_si128(Data + 1);

		Block0 = _mm_xor_si128(Block0, EKey[0]);
		Block1 = _mm_xor_si128(Block1, EKey[0]);
		Block0 = _mm_aesenc_si128(Block0, EKey[1]);
		Block1 = _mm_aesenc_si128(Block1, EKey[1]);
		Block0 = _mm_aesenc_si128(Block0, EKey[2]);
		Block1 = _mm_aesenc_si128(Block1, EKey[2]);
		Block0 = _mm_aesenc_si128(Block0, EKey[3]);
		Block1 = _mm_aesenc_si128(Block1, EKey[3]);
		Block0 = _mm_aesenc_si128(Block0, EKey[4]);
		Block1 = _mm_aesenc_si128(Block1, EKey[4]);
		Block0 = _mm_aesenc_si128(Block0, EKey[5]);
		Block1 = _mm_aesenc_si128(Block1, EKey[5]);
		Block0 = _mm_aesenc_si128(Block0, EKey[6]);
		Block1 = _mm_aesenc_si128(Block1, EKey[6]);
		Block0 = _mm_aesenc_si128(Block0, EKey[7]);
		Block1 = _mm_aesenc_si128(Block1, EKey[7]);
		Block0 = _mm_aesenc_si128(Block0, EKey[8]);
		Block1 = _mm_aesenc_si128(Block1, EKey[8]);
		Block0 = _mm_aesenc_si128(Block0, EKey[9]);
		Block1 = _mm_aesenc_si128(Block1, EKey[9]);
		Block0 = _mm_aesenc_si128(Block0, EKey[10]);
		Block1 = _mm_aesenc_si128(Block1, EKey[10]);
		Block0 = _mm_aesenc_si128(Block0, EKey[11]);
		Block1 = _mm_aesenc_si128(Block1, EKey[11]);
		Block0 = _mm_aesenc_si128(Block0, EKey[12]);
		Block1 = _mm_aesenc_si128(Block1, EKey[12]);
		Block0 = _mm_aesenc_si128(Block0, EKey[13]);
		Block1 = _mm_aesenc_si128(Block1, EKey[13]);
		Block0 = _mm_aesenclast_si128(Block0, EKey[14]);
		Block1 = _mm_aesenclast_si128(Block1, EKey[14]);

		_mm_storeu_si128(Data + 0, Block0);
		_mm_storeu_si128(Data + 1, Block1);
		Data += 2;
		NumBytes -= 32;
	}

	if (NumBytes != 0)
	{
		__m128i Block = _mm_loadu_si128(Data);

		Block = _mm_xor_si128(Block, EKey[0]);
		Block = _mm_aesenc_si128(Block, EKey[1]);
		Block = _mm_aesenc_si128(Block, EKey[2]);
		Block = _mm_aesenc_si128(Block, EKey[3]);
		Block = _mm_aesenc_si128(Block, EKey[4]);
		Block = _mm_aesenc_si128(Block, EKey[5]);
		Block = _mm_aesenc_si128(Block, EKey[6]);
		Block = _mm_aesenc_si128(Block, EKey[7]);
		Block = _mm_aesenc_si128(Block, EKey[8]);
		Block = _mm_aesenc_si128(Block, EKey[9]);
		Block = _mm_aesenc_si128(Block, EKey[10]);
		Block = _mm_aesenc_si128(Block, EKey[11]);
		Block = _mm_aesenc_si128(Block, EKey[12]);
		Block = _mm_aesenc_si128(Block, EKey[13]);
		Block = _mm_aesenclast_si128(Block, EKey[14]);

		_mm_storeu_si128(Data, Block);
	}
}

static inline void AesDecryptX86HW(const uint8* Key, uint8* Contents, uint64 NumBytes)
{
	const __m128i* Key128 = reinterpret_cast<const __m128i*>(Key);
	__m128i x = _mm_loadu_si128(Key128 + 0);
	__m128i y = _mm_loadu_si128(Key128 + 1);

	__m128i DKey[AES256_ROUND_COUNT + 1];
	DKey[14] = x;
	AES_INIT_DEC_X86(DKey[13], x, y, 0x01);
	AES_INIT_DEC_X86(DKey[12], y, x, 0x00);
	AES_INIT_DEC_X86(DKey[11], x, y, 0x02);
	AES_INIT_DEC_X86(DKey[10], y, x, 0x00);
	AES_INIT_DEC_X86(DKey[ 9], x, y, 0x04);
	AES_INIT_DEC_X86(DKey[ 8], y, x, 0x00);
	AES_INIT_DEC_X86(DKey[ 7], x, y, 0x08);
	AES_INIT_DEC_X86(DKey[ 6], y, x, 0x00);
	AES_INIT_DEC_X86(DKey[ 5], x, y, 0x10);
	AES_INIT_DEC_X86(DKey[ 4], y, x, 0x00);
	AES_INIT_DEC_X86(DKey[ 3], x, y, 0x20);
	AES_INIT_DEC_X86(DKey[ 2], y, x, 0x00);
	AES_INIT_DEC_X86(DKey[ 1], x, y, 0x40);
	DKey[0] = x;

	__m128i* Data = reinterpret_cast<__m128i*>(Contents);
	while (NumBytes >= 32)
	{
		__m128i Block0 = _mm_loadu_si128(Data + 0);
		__m128i Block1 = _mm_loadu_si128(Data + 1);

		Block0 = _mm_xor_si128(Block0, DKey[0]);
		Block1 = _mm_xor_si128(Block1, DKey[0]);
		Block0 = _mm_aesdec_si128(Block0, DKey[1]);
		Block1 = _mm_aesdec_si128(Block1, DKey[1]);
		Block0 = _mm_aesdec_si128(Block0, DKey[2]);
		Block1 = _mm_aesdec_si128(Block1, DKey[2]);
		Block0 = _mm_aesdec_si128(Block0, DKey[3]);
		Block1 = _mm_aesdec_si128(Block1, DKey[3]);
		Block0 = _mm_aesdec_si128(Block0, DKey[4]);
		Block1 = _mm_aesdec_si128(Block1, DKey[4]);
		Block0 = _mm_aesdec_si128(Block0, DKey[5]);
		Block1 = _mm_aesdec_si128(Block1, DKey[5]);
		Block0 = _mm_aesdec_si128(Block0, DKey[6]);
		Block1 = _mm_aesdec_si128(Block1, DKey[6]);
		Block0 = _mm_aesdec_si128(Block0, DKey[7]);
		Block1 = _mm_aesdec_si128(Block1, DKey[7]);
		Block0 = _mm_aesdec_si128(Block0, DKey[8]);
		Block1 = _mm_aesdec_si128(Block1, DKey[8]);
		Block0 = _mm_aesdec_si128(Block0, DKey[9]);
		Block1 = _mm_aesdec_si128(Block1, DKey[9]);
		Block0 = _mm_aesdec_si128(Block0, DKey[10]);
		Block1 = _mm_aesdec_si128(Block1, DKey[10]);
		Block0 = _mm_aesdec_si128(Block0, DKey[11]);
		Block1 = _mm_aesdec_si128(Block1, DKey[11]);
		Block0 = _mm_aesdec_si128(Block0, DKey[12]);
		Block1 = _mm_aesdec_si128(Block1, DKey[12]);
		Block0 = _mm_aesdec_si128(Block0, DKey[13]);
		Block1 = _mm_aesdec_si128(Block1, DKey[13]);
		Block0 = _mm_aesdeclast_si128(Block0, DKey[14]);
		Block1 = _mm_aesdeclast_si128(Block1, DKey[14]);

		_mm_storeu_si128(Data + 0, Block0);
		_mm_storeu_si128(Data + 1, Block1);
		Data += 2;
		NumBytes -= 32;
	}

	if (NumBytes != 0)
	{
		__m128i Block = _mm_loadu_si128(Data);

		Block = _mm_xor_si128(Block, DKey[0]);
		Block = _mm_aesdec_si128(Block, DKey[1]);
		Block = _mm_aesdec_si128(Block, DKey[2]);
		Block = _mm_aesdec_si128(Block, DKey[3]);
		Block = _mm_aesdec_si128(Block, DKey[4]);
		Block = _mm_aesdec_si128(Block, DKey[5]);
		Block = _mm_aesdec_si128(Block, DKey[6]);
		Block = _mm_aesdec_si128(Block, DKey[7]);
		Block = _mm_aesdec_si128(Block, DKey[8]);
		Block = _mm_aesdec_si128(Block, DKey[9]);
		Block = _mm_aesdec_si128(Block, DKey[10]);
		Block = _mm_aesdec_si128(Block, DKey[11]);
		Block = _mm_aesdec_si128(Block, DKey[12]);
		Block = _mm_aesdec_si128(Block, DKey[13]);
		Block = _mm_aesdeclast_si128(Block, DKey[14]);

		_mm_storeu_si128(Data, Block);
	}
}

#if PLATFORM_COMPILER_CLANG
#pragma clang attribute pop
#endif

#endif // PLATFORM_CPU_X86_FAMILY

#if PLATFORM_CPU_ARM_FAMILY

struct FAesExpandedKeyArm
{
	uint8x16_t Key[AES256_ROUND_COUNT + 1];
};

#if PLATFORM_COMPILER_CLANG
#pragma clang attribute push (__attribute__((target("aes"))), apply_to=function)
#endif

// Unfortunately, ARM AES intrinsics are preprocessor-gated and not target-gated in Android NDK 25.
// https://github.com/llvm/llvm-project/issues/56480
#if defined(__NDK_MAJOR__) && (__NDK_MAJOR__ <= 25) && !defined(__ARM_FEATURE_AES)
#	define vaeseq_u8(p0, p1) __builtin_neon_vaeseq_v(p0, p1, 48)
#	define vaesdq_u8(p0, p1) __builtin_neon_vaesdq_v(p0, p1, 48)
#	define vaesimcq_u8(p0)   __builtin_neon_vaesimcq_v(p0, 48)
#	define vaesmcq_u8(p0)    __builtin_neon_vaesmcq_v(p0, 48)
#endif

static inline uint32 AesEncryptMixArm(uint32 x)
{
	uint8x16_t Block = vreinterpretq_u8_u32(vmovq_n_u32(x));
	Block = vaeseq_u8(Block, vdupq_n_u8(0));
	return vgetq_lane_u32(vreinterpretq_u32_u8(Block), 0);
}

static inline void AesEncryptExpandArm(FAesExpandedKeyArm* EncryptKey, const uint8* Key)
{
	uint32 K[4 * (AES256_ROUND_COUNT + 1)];

	uint32* EKey = K;
	FMemory::Memcpy(EKey, Key, FAES::FAESKey::KeySize);

	for (int Index = 0; Index < AES256_ROUND_COUNT / 2; Index++, EKey += 8)
	{
		// rcon[] = { 0x01, 0x02, 0x04, 0x08, 0x10, 0x20, 0x40, 0x80 }
		uint32 Rcon = 1U << Index;

		EKey[ 8] = EKey[0] ^ RotateRight(AesEncryptMixArm(EKey[7]), 8) ^ Rcon;
		EKey[ 9] = EKey[1] ^ EKey[ 8];
		EKey[10] = EKey[2] ^ EKey[ 9];
		EKey[11] = EKey[3] ^ EKey[10];

		if (Index < AES256_ROUND_COUNT / 2 - 1)
		{
			EKey[12] = EKey[4] ^ AesEncryptMixArm(EKey[11]);
			EKey[13] = EKey[5] ^ EKey[12];
			EKey[14] = EKey[6] ^ EKey[13];
			EKey[15] = EKey[7] ^ EKey[14];
		}
	}

	for (int Round = 0; Round <= AES256_ROUND_COUNT; Round++)
	{
		EncryptKey->Key[Round] = vreinterpretq_u8_u32(vld1q_u32(K + Round * 4));
	}
}

static inline void AesDecryptExpandArm(FAesExpandedKeyArm* DecryptKey, const uint8* Key)
{
	FAesExpandedKeyArm EncryptKey;
	AesEncryptExpandArm(&EncryptKey, Key);

	DecryptKey->Key[0] = EncryptKey.Key[AES256_ROUND_COUNT];
	for (int Round = 1; Round < AES256_ROUND_COUNT; Round++)
	{
		DecryptKey->Key[Round] = vaesimcq_u8(EncryptKey.Key[AES256_ROUND_COUNT - Round]);
	}
	DecryptKey->Key[AES256_ROUND_COUNT] = EncryptKey.Key[0];
}

static inline void AesEncryptArmHW(const uint8* Key, uint8* Contents, uint64 NumBytes)
{
	FAesExpandedKeyArm EncryptKey;
	AesEncryptExpandArm(&EncryptKey, Key);

	const uint8x16_t* EKey = EncryptKey.Key;

	for (uint64 Offset = 0; Offset < NumBytes; Offset += 16)
	{
		uint8* Data = Contents + Offset;

		uint8x16_t Block = vld1q_u8(Data);
		Block = vaesmcq_u8(vaeseq_u8(Block, EKey[0]));
		Block = vaesmcq_u8(vaeseq_u8(Block, EKey[1]));
		Block = vaesmcq_u8(vaeseq_u8(Block, EKey[2]));
		Block = vaesmcq_u8(vaeseq_u8(Block, EKey[3]));
		Block = vaesmcq_u8(vaeseq_u8(Block, EKey[4]));
		Block = vaesmcq_u8(vaeseq_u8(Block, EKey[5]));
		Block = vaesmcq_u8(vaeseq_u8(Block, EKey[6]));
		Block = vaesmcq_u8(vaeseq_u8(Block, EKey[7]));
		Block = vaesmcq_u8(vaeseq_u8(Block, EKey[8]));
		Block = vaesmcq_u8(vaeseq_u8(Block, EKey[9]));
		Block = vaesmcq_u8(vaeseq_u8(Block, EKey[10]));
		Block = vaesmcq_u8(vaeseq_u8(Block, EKey[11]));
		Block = vaesmcq_u8(vaeseq_u8(Block, EKey[12]));
		Block = vaeseq_u8(Block, EKey[13]);
		Block = veorq_u8(Block, EKey[14]);
		vst1q_u8(Data, Block);
	}
}

static inline void AesDecryptArmHW(const uint8* Key, uint8* Contents, uint64 NumBytes)
{
	FAesExpandedKeyArm DecryptKey;
	AesDecryptExpandArm(&DecryptKey, Key);

	const uint8x16_t* DKey = DecryptKey.Key;

	for (uint64 Offset = 0; Offset < NumBytes; Offset += 16)
	{
		uint8* Data = Contents + Offset;

		uint8x16_t Block = vld1q_u8(Data);
		Block = vaesimcq_u8(vaesdq_u8(Block, DKey[0]));
		Block = vaesimcq_u8(vaesdq_u8(Block, DKey[1]));
		Block = vaesimcq_u8(vaesdq_u8(Block, DKey[2]));
		Block = vaesimcq_u8(vaesdq_u8(Block, DKey[3]));
		Block = vaesimcq_u8(vaesdq_u8(Block, DKey[4]));
		Block = vaesimcq_u8(vaesdq_u8(Block, DKey[5]));
		Block = vaesimcq_u8(vaesdq_u8(Block, DKey[6]));
		Block = vaesimcq_u8(vaesdq_u8(Block, DKey[7]));
		Block = vaesimcq_u8(vaesdq_u8(Block, DKey[8]));
		Block = vaesimcq_u8(vaesdq_u8(Block, DKey[9]));
		Block = vaesimcq_u8(vaesdq_u8(Block, DKey[10]));
		Block = vaesimcq_u8(vaesdq_u8(Block, DKey[11]));
		Block = vaesimcq_u8(vaesdq_u8(Block, DKey[12]));
		Block = vaesdq_u8(Block, DKey[13]);
		Block = veorq_u8(Block, DKey[14]);
		vst1q_u8(Data, Block);
	}
}

#if PLATFORM_COMPILER_CLANG
#pragma clang attribute pop
#endif

#endif // PLATFORM_CPU_ARM_FAMILY

void FAES::EncryptData(uint8* Contents, uint64 NumBytes, const FAESKey& Key)
{
	checkf(Key.IsValid(), TEXT("No valid encryption key specified"));
	EncryptData(Contents, NumBytes, Key.Key, sizeof(Key.Key));
}

void FAES::EncryptData(uint8* Contents, uint64 NumBytes, const ANSICHAR* Key)
{
	checkf(Key != nullptr, TEXT("No encryption key specified"));
	EncryptData(Contents, NumBytes, (const uint8*)Key, TCString<ANSICHAR>::Strlen(Key));
}

void FAES::EncryptData(uint8* Contents, uint64 NumBytes, const uint8* KeyBytes, uint32 NumKeyBytes)
{
	checkf(NumBytes % AESBlockSize == 0, TEXT("NumBytes needs to be a multiple of 16 bytes"));
	checkf(NumKeyBytes == FAESKey::KeySize, TEXT("AES-256 key needs to be 32 bytes long"));
	AesEncrypt(KeyBytes, Contents, NumBytes);
}

void FAES::DecryptData(uint8* Contents, uint64 NumBytes, const FAESKey& Key)
{
	checkf(Key.IsValid(), TEXT("No valid decryption key specified"));
	DecryptData(Contents, NumBytes, Key.Key, sizeof(Key.Key));
}

void FAES::DecryptData(uint8* Contents, uint64 NumBytes, const ANSICHAR* Key)
{
	checkf(Key != nullptr, TEXT("No valid decryption key specified"));
	DecryptData(Contents, NumBytes, (const uint8*)Key, TCString<ANSICHAR>::Strlen(Key));
}

void FAES::DecryptData(uint8* Contents, uint64 NumBytes, const uint8* KeyBytes, uint32 NumKeyBytes)
{
	checkf(NumBytes % AESBlockSize == 0, TEXT("NumBytes needs to be a multiple of 16 bytes"));
	checkf(NumKeyBytes == FAESKey::KeySize, TEXT("AES-256 key needs to be 32 bytes long"));
	AesDecrypt(KeyBytes, Contents, NumBytes);
}