// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AccessDetection.h"
#include "Misc/AssertionMacros.h"
#include "Templates/Atomic.h"

#if WITH_EDITOR
namespace UE { namespace AccessDetection { namespace Private {

volatile int32 GNumLiveScopes;
thread_local FScope* TScope;

void SetCurrentThreadScope(FScope* Scope)
{
	checkf(!!Scope != !TScope, TEXT("Nested access detection scopes are not allowed"));
	FPlatformAtomics::InterlockedAdd(&GNumLiveScopes, Scope ? 1 : -1);
	TScope = Scope;
}

void ReportCurrentThreadAccess(EType Type)
{
	if (FScope* Scope = TScope)
	{
		Scope->AddAccess(Type);	
	}
}

}}}
#endif