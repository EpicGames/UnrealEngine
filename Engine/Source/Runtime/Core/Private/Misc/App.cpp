// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/App.h"
#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreDelegates.h"
#include "Runtime/Launch/Resources/Version.h"
#include "Serialization/CompactBinary.h"
#include "HAL/LowLevelMemTracker.h"
#include "BuildSettings.h"
#include "UObject/DevObjectVersion.h"
#include "Misc/EngineVersion.h"
#include "Misc/NetworkVersion.h"
#include "Misc/SecureHash.h"

#ifndef UE_DEFAULT_MULTITHREAD_SERVER
	// Controls if the dedicated server should enable multithreading or not
	#define UE_DEFAULT_MULTITHREAD_SERVER 0
#endif

DEFINE_LOG_CATEGORY_STATIC(LogApp, Log, All);

/* FApp static initialization
 *****************************************************************************/

#if UE_BUILD_DEVELOPMENT
bool FApp::bIsDebugGame = false;
#endif

FGuid FApp::InstanceId = FGuid::NewGuid();
FGuid FApp::SessionId = ToGuid(FApp::GetSessionObjectId());
FString FApp::SessionName = FString();
FString FApp::SessionOwner = FString();
FString FApp::GraphicsRHI = FString();
TArray<FString> FApp::SessionUsers = TArray<FString>();
bool FApp::Standalone = true;
bool FApp::bIsBenchmarking = false;
bool FApp::bUseFixedSeed = false;
float FApp::VolumeMultiplier = 1.0f;
float FApp::UnfocusedVolumeMultiplier = 0.0f;
bool FApp::bUseVRFocus = false;
bool FApp::bHasVRFocus = false;
bool (*FApp::HasFocusFunction)() = nullptr;


/* FApp static interface
 *****************************************************************************/

FString FApp::GetBranchName()
{
	return FString(BuildSettings::GetBranchName());
}

const TCHAR* FApp::GetBuildVersion()
{
	return BuildSettings::GetBuildVersion();
}

const TCHAR* FApp::GetBuildURL()
{
	if (FCoreDelegates::OnGetBuildURL.IsBound()) 
	{
		return FCoreDelegates::OnGetBuildURL.Execute();
	}	
	return BuildSettings::GetBuildURL();
}	

int32 FApp::GetEngineIsPromotedBuild()
{
	return BuildSettings::IsPromotedBuild()? 1 : 0;
}

bool FApp::GetIsWithDebugInfo()
{
	return BuildSettings::IsWithDebugInfo();
}

const TCHAR* FApp::GetExecutingJobURL()
{
	if (FCoreDelegates::OnGetExecutingJobURL.IsBound())
	{
		return FCoreDelegates::OnGetExecutingJobURL.Execute();
	}

	static const FString URL = []() -> FString {
		FString HordeUrl = FPlatformMisc::GetEnvironmentVariable(TEXT("UE_HORDE_URL"));
		if (HordeUrl.IsEmpty())
		{
			return FString();
		}
		else
		{
			HordeUrl.RemoveFromEnd(TEXT("/"));
		}
		FString HordeJobId = FPlatformMisc::GetEnvironmentVariable(TEXT("UE_HORDE_JOBID"));
		if (HordeJobId.IsEmpty())
		{
			return FString();
		}
		FString HordeStepId = FPlatformMisc::GetEnvironmentVariable(TEXT("UE_HORDE_STEPID"));
		if (HordeStepId.IsEmpty())
		{
			return FString::Printf(TEXT("%s/job/%s"), *HordeUrl, *HordeJobId);
		}
		else 
		{
			return FString::Printf(TEXT("%s/job/%s?step=%s"), *HordeUrl, *HordeJobId, *HordeStepId);
		}
	}();
	return *URL;
}

FString FApp::GetEpicProductIdentifier()
{
	return FString(TEXT(EPIC_PRODUCT_IDENTIFIER));
}

EBuildConfiguration FApp::GetBuildConfiguration()
{
#if UE_BUILD_DEBUG
	return EBuildConfiguration::Debug;

#elif UE_BUILD_DEVELOPMENT
	return bIsDebugGame ? EBuildConfiguration::DebugGame : EBuildConfiguration::Development;

#elif UE_BUILD_SHIPPING
	return EBuildConfiguration::Shipping;

#elif UE_BUILD_TEST
	return EBuildConfiguration::Test;

#else
	return EBuildConfiguration::Unknown;
#endif
}

EBuildTargetType FApp::GetBuildTargetType()
{
#if IS_CLIENT_TARGET
	return EBuildTargetType::Client;
#elif UE_GAME
	return EBuildTargetType::Game;
#elif UE_EDITOR
	return EBuildTargetType::Editor;
#elif UE_SERVER
	return EBuildTargetType::Server;
#elif IS_PROGRAM
	return EBuildTargetType::Program;
#else
	static_assert(false, "No host type is set.");
#endif
}

#if UE_BUILD_DEVELOPMENT
void FApp::SetDebugGame(bool bInIsDebugGame)
{
	bIsDebugGame = bInIsDebugGame;
}
#endif

FString FApp::GetBuildDate()
{
	return FString(BuildSettings::GetBuildDate());
}

FString FApp::GetGraphicsRHI()
{
	return GraphicsRHI;
}

void FApp::SetGraphicsRHI(FString RHIString)
{
	GraphicsRHI = RHIString;
}

void FApp::InitializeSession()
{
	// parse session details on command line
	FString InstanceIdString;

	if (FParse::Value(FCommandLine::Get(), TEXT("-InstanceId="), InstanceIdString))
	{
		if (!FGuid::Parse(InstanceIdString, InstanceId))
		{
			UE_LOGF(LogInit, Warning, "Invalid InstanceId on command line: %ls", *InstanceIdString);
		}
	}

	FString SessionIdString;
	if (FParse::Value(FCommandLine::Get(), TEXT("-SessionId="), SessionIdString))
	{
		if (FGuid::Parse(SessionIdString, SessionId))
		{
			Standalone = false;
		}
		else
		{
			UE_LOGF(LogInit, Warning, "Invalid SessionId on command line: %ls", *SessionIdString);
		}
	}

	FParse::Value(FCommandLine::Get(), TEXT("-SessionName="), SessionName);

	if (!FParse::Value(FCommandLine::Get(), TEXT("-SessionOwner="), SessionOwner))
	{
		SessionOwner = FPlatformProcess::UserName(false);
	}
}


bool FApp::IsInstalled()
{
	static int32 InstalledState = -1;

	if (InstalledState == -1)
	{
#if UE_BUILD_SHIPPING && PLATFORM_DESKTOP && !UE_SERVER
		bool bIsInstalled = true;
#else
		bool bIsInstalled = false;
#endif

#if PLATFORM_DESKTOP
		FString InstalledProjectBuildFile = FPaths::RootDir() / TEXT("Engine/Build/InstalledProjectBuild.txt");
		FPaths::NormalizeFilename(InstalledProjectBuildFile);
		bIsInstalled |= IFileManager::Get().FileExists(*InstalledProjectBuildFile);
#endif

		// Allow commandline options to disable/enable installed engine behavior
		if (bIsInstalled)
		{
			bIsInstalled = !FParse::Param(FCommandLine::Get(), TEXT("NotInstalled"));
		}
		else
		{
			bIsInstalled = FParse::Param(FCommandLine::Get(), TEXT("Installed"));
		}
		InstalledState = bIsInstalled ? 1 : 0;
	}
	return InstalledState == 1;
}


bool FApp::IsEngineInstalled()
{
	static int32 EngineInstalledState = -1;

	if (EngineInstalledState == -1)
	{
		bool bIsInstalledEngine = IsInstalled();

#if PLATFORM_DESKTOP
		FString InstalledBuildFile = FPaths::RootDir() / TEXT("Engine/Build/InstalledBuild.txt");
		FPaths::NormalizeFilename(InstalledBuildFile);
		bIsInstalledEngine |= IFileManager::Get().FileExists(*InstalledBuildFile);
#endif

		// Allow commandline options to disable/enable installed engine behavior
		if (bIsInstalledEngine)
		{
			bIsInstalledEngine = !FParse::Param(FCommandLine::Get(), TEXT("NotInstalledEngine"));
		}
		else
		{
			bIsInstalledEngine = FParse::Param(FCommandLine::Get(), TEXT("InstalledEngine"));
		}
		EngineInstalledState = bIsInstalledEngine ? 1 : 0;
	}

	return EngineInstalledState == 1;
}

bool FApp::IsUnattended()
{
	// FCommandLine::Get() will assert that the command line has been set.
	// This function may not be used before FCommandLine::Set() is called.
	static bool bIsUnattended = FParse::Param(FCommandLine::Get(), TEXT("UNATTENDED"));
	return bIsUnattended || GIsAutomationTesting;
}

bool FApp::AllowUnattendedInput()
{
	// FCommandLine::Get() will assert that the command line has been set.
	// This function may not be used before FCommandLine::Set() is called.
	static bool bUnattendedInput = FParse::Param(FCommandLine::Get(), TEXT("UnattendedInput"));
	return bUnattendedInput;
}

bool FApp::ShouldUseThreadingForPerformance()
{
	static bool OnlyOneThread = 
		FParse::Param(FCommandLine::Get(), TEXT("onethread")) ||
		FParse::Param(FCommandLine::Get(), TEXT("noperfthreads")) ||
		IsRunningDedicatedServer() ||
		!FPlatformProcess::SupportsMultithreading() ||
		FPlatformMisc::NumberOfCoresIncludingHyperthreads() == 1;

	// Enable at runtime for experimentation by passing "useperfthreads" as a command line arg.
	static bool bForceUsePerfThreads = FParse::Param(FCommandLine::Get(), TEXT("useperfthreads"));
	return !OnlyOneThread || bForceUsePerfThreads;
}

bool FApp::IsMultithreadServer()
{
	static bool bIsMultithreadServer = IsRunningDedicatedServer() && UE_DEFAULT_MULTITHREAD_SERVER;
	return bIsMultithreadServer;
}

static bool GUnfocusedVolumeMultiplierInitialised = false;
float FApp::GetUnfocusedVolumeMultiplier()
{
	if (!GUnfocusedVolumeMultiplierInitialised)
	{
		GUnfocusedVolumeMultiplierInitialised = true;
		GConfig->GetFloat(TEXT("Audio"), TEXT("UnfocusedVolumeMultiplier"), UnfocusedVolumeMultiplier, GEngineIni);
	}
	return UnfocusedVolumeMultiplier;
}

void FApp::SetUnfocusedVolumeMultiplier(float InVolumeMultiplier)
{
	UnfocusedVolumeMultiplier = InVolumeMultiplier;
	GConfig->SetFloat(TEXT("Audio"), TEXT("UnfocusedVolumeMultiplier"), UnfocusedVolumeMultiplier, GEngineIni);
	GUnfocusedVolumeMultiplierInitialised = true;
}

void FApp::SetUseVRFocus(bool bInUseVRFocus)
{
	UE_CLOGF(bUseVRFocus != bInUseVRFocus, LogApp, Verbose, "UseVRFocus has changed to %d", int(bInUseVRFocus));
	bUseVRFocus = bInUseVRFocus;
}

void FApp::SetHasVRFocus(bool bInHasVRFocus)
{
	UE_CLOGF(bHasVRFocus != bInHasVRFocus, LogApp, Verbose, "HasVRFocus has changed to %d", int(bInHasVRFocus));
	bHasVRFocus = bInHasVRFocus;
}

void FApp::SetHasFocusFunction(bool (*InHasFocusFunction)())
{
	HasFocusFunction = InHasFocusFunction;
}

bool FApp::HasFocus()
{
	if (FApp::IsBenchmarking())
	{
		return true;
	}

	if (FApp::UseVRFocus())
	{
		return FApp::HasVRFocus();
	}

	// by default we assume we have focus, it's a worse thing to encounter a bug where focus is locked off, vs. locked on
	bool bHasFocus = true;

	// desktop platforms are more or less why we have this abstraction, to dip into ApplicationCore's Platform implementation
#if PLATFORM_DESKTOP
	check(HasFocusFunction);
#endif

	// call the HasFocusFunction, if we have one. otherwise fall back to the default
	return HasFocusFunction ? HasFocusFunction() : bHasFocus;
}

void FApp::PrintStartupLogMessages()
{
	UE_LOGF(LogInit, Log, "ExecutableName: %ls", FPlatformProcess::ExecutableName(false));
	UE_LOGF(LogInit, Log, "Build: %ls", FApp::GetBuildVersion());

	UE_LOGF(LogInit, Log, "Platform=%ls", ANSI_TO_TCHAR(FPlatformProperties::PlatformName()));
	UE_LOGF(LogInit, Log, "MachineId=%ls", *FPlatformMisc::GetLoginId());
	UE_LOGF(LogInit, Log, "DeviceId=%ls", *FPlatformMisc::GetDeviceId());

	UE_LOGF(LogInit, Log, "Engine Version: %ls", *FEngineVersion::Current().ToString());
	UE_LOGF(LogInit, Log, "Compatible Engine Version: %ls", *FEngineVersion::CompatibleWith().ToString());
	UE_LOGF(LogInit, Log, "Net CL: %u", FNetworkVersion::GetNetworkCompatibleChangelist());
	FString OSLabel, OSVersion;
	FPlatformMisc::GetOSVersions(OSLabel, OSVersion);
	UE_LOGF(LogInit, Log, "OS: %ls (%ls), CPU: %ls, GPU: %ls", *OSLabel, *OSVersion, *FPlatformMisc::GetCPUBrand(), *FPlatformMisc::GetPrimaryGPUBrand());
	UE_LOGF(LogInit, Log, "CPU Model ID: %x, microcode revision: %x", FPlatformMisc::GetCPUModelID(), FPlatformMisc::GetCPUMicrocodeRevision());

	UE_LOGF(LogInit, Log, "Compiled (64-bit): %ls %ls", BuildSettings::GetBuildDate(), BuildSettings::GetBuildTime());

#if PLATFORM_CPU_ARM_FAMILY
	UE_LOGF(LogInit, Log, "Architecture: arm64");
#elif PLATFORM_CPU_X86_FAMILY
	UE_LOGF(LogInit, Log, "Architecture: x64");
#elif defined(UE_ARCH_NAME)
	UE_LOGF(LogInit, Log, "Architecture: %s", UE_STRINGIZE(UE_ARCH_NAME));
#else
#error No architecture name defined!
#endif // x64/arm

	// Print compiler version info
#if defined(__clang__)
	UE_LOGF(LogInit, Log, "Compiled with Clang: %ls", ANSI_TO_TCHAR(__clang_version__));
#elif defined(__INTEL_COMPILER)
	UE_LOGF(LogInit, Log, "Compiled with ICL: %d", __INTEL_COMPILER);
#elif defined( _MSC_VER )
#ifndef __INTELLISENSE__	// Intellisense compiler doesn't support _MSC_FULL_VER
	{
		const FString VisualCPPVersion(FString::Printf(TEXT("%d"), _MSC_FULL_VER));
		const FString VisualCPPRevisionNumber(FString::Printf(TEXT("%02d"), _MSC_BUILD));
		UE_LOGF(LogInit, Log, "Compiled with Visual C++: %ls.%ls.%ls.%ls",
			*VisualCPPVersion.Mid(0, 2), // Major version
			*VisualCPPVersion.Mid(2, 2), // Minor version
			*VisualCPPVersion.Mid(4),	// Build version
			*VisualCPPRevisionNumber	// Revision number
		);
	}
#endif
#else
	UE_LOGF(LogInit, Log, "Compiled with unrecognized C++ compiler");
#endif

	UE_LOGF(LogInit, Log, "Build Configuration: %ls", LexToString(FApp::GetBuildConfiguration()));
	UE_LOGF(LogInit, Log, "Branch Name: %ls", *FApp::GetBranchName());
	FString FilteredString = FCommandLine::IsCommandLineLoggingFiltered() ? TEXT("Filtered ") : TEXT("");
	UE_LOGF(LogInit, Log, "%lsCommand Line: %ls", *FilteredString, FCommandLine::GetForLogging());
	UE_LOGF(LogInit, Log, "Base Directory: %ls", FPlatformProcess::BaseDir());
	//UE_LOGF(LogInit, Log, "Character set: %ls", sizeof(TCHAR)==1 ? TEXT("ANSI") : TEXT("Unicode") );
	UE_LOGF(LogInit, Log, "Allocator: %ls", UE::Private::GMalloc->GetDescriptiveName());
	UE_LOGF(LogInit, Log, "Installed Engine Build: %d", FApp::IsEngineInstalled() ? 1 : 0);
	UE_LOGF(LogInit, Log, "This binary is optimized with LTO: %ls, PGO: %ls, instrumented for PGO data collection: %ls",
		PLATFORM_COMPILER_OPTIMIZATION_LTCG ? TEXT("yes") : TEXT("no"),
		FPlatformMisc::IsPGOEnabled() ? TEXT("yes") : TEXT("no"),
		FPlatformMisc::IsPGICapableBinary() ? TEXT("yes") : TEXT("no")
	);

	FDevVersionRegistration::DumpVersionsToLog();
}

FString FApp::GetZenStoreProjectId(FStringView SubProject)
{
	FString ProjectId;
	if (FParse::Value(FCommandLine::Get(), TEXT("-ZenStoreProject="), ProjectId))
	{
		return ProjectId;
	}

#if PLATFORM_DESKTOP
	if (FPaths::IsProjectFilePathSet())
	{
		return GetZenStoreProjectIdForProject(FPaths::GetProjectFilePath(), SubProject);
	}
	UE_LOGF(LogInit, Fatal, "GetZenStoreProjectId() called before having a valid project file path");
#else
	UE_LOGF(LogInit, Fatal, "-ZenStoreProject command line argument is required to run from Zen");
#endif
	return FString();
}

#if PLATFORM_DESKTOP
FString FApp::GetZenStoreProjectIdForProject(FStringView ProjectFilePath, FStringView SubProject)
{
	if (ProjectFilePath.IsEmpty())
	{
		return FString();
	}

	FString ProjectFilePathStr(ProjectFilePath);
	FString AbsProjectFilePath = IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*ProjectFilePathStr);
	AbsProjectFilePath = FPaths::FindCorrectCase(AbsProjectFilePath);
	FTCHARToUTF8 AbsProjectFilePathUTF8(*AbsProjectFilePath);

	FString HashString = FMD5::HashBytes((unsigned char*)AbsProjectFilePathUTF8.Get(), AbsProjectFilePathUTF8.Length()).Left(8);
	FString ProjectName = FPaths::GetBaseFilename(ProjectFilePathStr);

	if (SubProject.IsEmpty())
	{
		return FString::Printf(TEXT("%s.%.8s"), *ProjectName, *HashString);
	}
	else
	{
		return FString::Printf(TEXT("%s.%.*s.%.8s"), *ProjectName, SubProject.Len(), SubProject.GetData(), *HashString);
	}
}
#endif


FGuid FApp::GetInstanceId()
{
	return InstanceId;
}

const FCbObjectId& FApp::GetSessionObjectId()
{
	static const FCbObjectId SessionObjectId = FCbObjectId::NewObjectId();
	return SessionObjectId;
}
