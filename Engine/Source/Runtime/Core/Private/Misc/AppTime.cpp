// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AppTime.h"
#include "CoreGlobals.h"

CORE_API FAppTime FAppTime::GameThreadInstance;
thread_local TSharedPtr<const FAppTime> FAppTime::TLSInstance;

[[nodiscard]] CORE_API TSharedPtr<const FAppTime> FAppTime::Fork()
{
	if (IsInSlateThread())
	{
		// @todo delta time refactor - ignore races with the slate thread for now
		return MakeShared<FAppTime>(GameThreadInstance);
	}

	return TLSInstance;
}

CORE_API void FAppTime::Restore(TSharedPtr<const FAppTime>&& Ptr)
{
	TLSInstance = MoveTemp(Ptr);
}

CORE_API const FAppTime& FAppTime::Get()
{
	if (TLSInstance.IsValid())
	{
		return *TLSInstance;
	}
	else if (IsInSlateThread() || IsInAsyncLoadingThread() || IsInParallelLoadingThread())
	{
		// @todo delta time refactor - ignore races with the slate thread and loading threads for now
		return GameThreadInstance;
	}
	else
	{
		ensureMsgf(IsInGameThread(),
			TEXT("Attempted to retrieve FAppTime on a thread where there is no inherited time context. "        )
			TEXT("Copies of the game thread's FAppTime are passed automatically through the TaskGraph via "     )
			TEXT("FInheritedContextBase / FInheritedContextScope, to ensure the values are pipelined correctly ")
			TEXT("to tasks that may be running concurrently. The current thread or task has not been directly " )
			TEXT("or indirectly spawned from the game thread, so it did not receive a time context."            ));

		return GameThreadInstance;
	}
}

CORE_API const FAppTime& FAppTime::Get_UnsafeNoCheck()
{
	// Prefer the TLS value if we have one.
	if (TLSInstance.IsValid())
	{
		return *TLSInstance;
	}

	// Unsynchronized read from the game thread's copy of FAppTime.
	// This is a data race if read from any thread except the game thread.
	return GameThreadInstance;
}

void FAppTime::Set(const FAppTime& NewValue)
{
	checkf(IsInGameThread(), TEXT("Only the game thread may set the current FAppTime value, to ensure parallel tasks receive the correct value via task context inheritance."));
	GameThreadInstance = NewValue;
	TLSInstance = MakeShared<FAppTime>(GameThreadInstance);
}
