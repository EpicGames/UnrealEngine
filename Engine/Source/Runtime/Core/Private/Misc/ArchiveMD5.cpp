// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/ArchiveMD5.h"

// Force a virtual function to be defined out-of-line so that the vtable
// is emitted in this module instead of everywhere FArchiveMD5 is used.
// Otherwise, it's causing a linking issue when used in a RTTI module
FString FArchiveMD5::GetArchiveName() const
{
	return TEXT("FArchiveMD5");
}
