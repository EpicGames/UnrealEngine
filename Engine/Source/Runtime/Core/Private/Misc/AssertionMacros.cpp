// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AssertionMacros.h"
#include "Misc/VarArgs.h"
#include "HAL/UnrealMemory.h"
#include "Templates/UnrealTemplate.h"
#include "Templates/Atomic.h"
#include "Misc/CString.h"
#include "Misc/Crc.h"
#include "Async/UniqueLock.h"
#include "Async/WordMutex.h"
#include "AutoRTFM.h"
#include "Containers/UnrealString.h"
#include "Containers/StringConv.h"
#include "GenericPlatform/GenericPlatformStackWalk.h"
#include "HAL/PlatformStackWalk.h"
#include "Logging/LogMacros.h"
#include "Logging/StructuredLog.h"
#include "CoreGlobals.h"
#include "Misc/Parse.h"
#include "Misc/ScopeLock.h"
#include "Misc/CoreMisc.h"
#include "Misc/CommandLine.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Misc/OutputDeviceError.h"
#include "Misc/StringBuilder.h"
#include "Stats/StatsMisc.h"
#include "Misc/CoreDelegates.h"
#include "HAL/ExceptionHandling.h"
#include "HAL/ThreadHeartBeat.h"
#include "HAL/IConsoleManager.h"
#include "AutoRTFM.h"

// Stack walking can race with ASAN's usage of DbgHelp (which is not thread safe), so we're
// not going to report callstacks for failed ensures and checks if ASAN is enabled.
#if PLATFORM_WINDOWS && PLATFORM_HAS_ASAN_INCLUDE
#	if defined(__SANITIZE_ADDRESS__)
#		define ASSERT_DISABLE_BACKTRACE_DUE_TO_ASAN 1
#	elif defined(__has_feature)
#		if __has_feature(address_sanitizer)
#			define ASSERT_DISABLE_BACKTRACE_DUE_TO_ASAN 1
#		endif
#	endif
#endif

#ifndef ASSERT_DISABLE_BACKTRACE_DUE_TO_ASAN
#	define ASSERT_DISABLE_BACKTRACE_DUE_TO_ASAN 0
#endif


namespace 
{
	// used to track state of assets/ensures
	bool bHasAsserted = false;
	TAtomic<SIZE_T> NumEnsureFailures {0};
	int32 ActiveEnsureCount = 0;

	// Lock used to synchronize the fail debug calls.
	// Using FWordMutex because it is zero-initialized and has no destructor.
	static UE::FWordMutex FailDebugMutex;

	struct FTempCommandLineScope
	{
		// The code which is run when an assert or ensure fails (without a
		// debugger attached) calls FCommandLine::Get() *a lot*. If the failed
		// assert is before a command line has been set then the many Get()
		// calls will in turn throw asserts.  It is impractical to chase these
		// and guard against calling Get() and inappropriate in many instances.
		FTempCommandLineScope()
		{
			if (!FCommandLine::IsInitialized())
			{
				FCommandLine::Set(TEXT(""));
				bShouldReset = true;
			}
		}

		~FTempCommandLineScope()
		{
			if (bShouldReset)
			{
				FCommandLine::Reset();
			}
		}

		bool bShouldReset = false;
	};
}

#define FILE_LINE_DESC_ANSI " [File:%hs] [Line: %i] "

/*
	Ensure behavior

	* ensure() macro calls OptionallyLogFormattedEnsureMessageReturningFalse 
	* OptionallyLogFormattedEnsureMessageReturningFalse calls EnsureFailed()
	* EnsureFailed() -
		* Formats the ensure failure and calls StaticFailDebugV to populate the global error info (without callstack)
		* Prints the script callstack (if any)
		* Halts if a debugger is attached 
		* If not, logs the callstack and attempts to submit an error report
	* execution continues as normal, (on some platforms this can take ~30 secs to perform)

	Check behavior

	* check() macro calls LogAssertFailedMessage
	* LogAssertFailedMessage formats the assertion message and calls StaticFailDebugV
	* StaticFailDebugV populates global error info with the failure message and if supported (AllowsCallStackDumpDuringAssert) the callstack
	* If a debugger is attached execution halts
	* If not FDebug::AssertFailed is called
	* FDebug::AssertFailed logs the assert message and description to GError
	* At this point behavior depends on the platform-specific error output device implementation
		* Desktop platforms (Windows, Mac, Linux) will generally throw an exception and in the handler attempt to submit a crash report and exit
		* Console platforms will generally dump the info to the log and abort()

	Fatal-error behavior

	* The UE_LOG macro calls FMsg::Logf which checks for "Fatal" verbosity
	* FMsg::Logf formats the failure message and calls StaticFailDebugV
	* StaticFailDebugV populates global error info with the failure message and if supported (AllowsCallStackDumpDuringAssert) the callstack
	* FDebug::AssertFailed is then called, and from this point behavior is identical to an assert but with a different message

*/

/** 
 * Use this CVar to control whether ensures count as errors or warnings. 
 * Errors will fail certain processes like cooks, whereas warnings will not.
 */
int32 GEnsuresAreErrors = 1;
FAutoConsoleVariableRef CVarEnsuresAreErrors(
	TEXT("core.EnsuresAreErrors"),
	GEnsuresAreErrors,
	TEXT("True means failed ensures are logged as errors. False means they are logged as warnings."),
	ECVF_Default);

bool GEnsureAlwaysEnabled = true;
FAutoConsoleVariableRef CVarEnsureAlwaysEnabled(
	TEXT("core.EnsureAlwaysEnabled"),
	GEnsureAlwaysEnabled,
	TEXT("Set to false to turn ensureAlways into regular ensure"),
	ECVF_Default);

bool GEnsureBreakEnabled = true;
FAutoConsoleVariableRef CVarEnsureBreakEnabled(
	TEXT("core.EnsureBreakEnabled"),
	GEnsureBreakEnabled,
	TEXT("Set to false to turn off the debugger catching ensures"),
	ECVF_Default);

int32 GEnsureResetState = 1;
FAutoConsoleCommand CVarResetEnsureState(
	TEXT("core.ResetEnsureState"),
	TEXT("Reset all ensures so they will fire again"),
	FConsoleCommandDelegate::CreateLambda([]()
		{
			// Increment and wrap to 0-255
			GEnsureResetState = (uint8)(GEnsureResetState + 1);
		}));

FAutoConsoleVariableRef CVarEnsureStateRef(
    TEXT("core.EnsureState"),
    GEnsureResetState,
    TEXT("Set the underlying value/counter for ensure, its initial value is 1"));

CORE_API void (*GPrintScriptCallStackFn)() = nullptr;

void PrintScriptCallstack()
{
	if(GPrintScriptCallStackFn)
	{
		GPrintScriptCallStackFn();
	}
}

static void AssertFailedImplV(const ANSICHAR* Expr, const ANSICHAR* File, int32 Line, void* ProgramCounter, const TCHAR* Format, va_list Args)
{
	FTempCommandLineScope TempCommandLine;

	// This is not perfect because another thread might crash and be handled before this assert
	// but this static variable will report the crash as an assert. Given complexity of a thread
	// aware solution, this should be good enough. If crash reports are obviously wrong we can
	// look into fixing this.
	bHasAsserted = true;

	if (GError)
	{
		TCHAR DescriptionString[4096];
		FCString::GetVarArgs(DescriptionString, UE_ARRAY_COUNT(DescriptionString), Format, Args);
		GError->SetErrorProgramCounter(ProgramCounter);
		GError->Logf(TEXT("Assertion failed: %hs" FILE_LINE_DESC_ANSI "\n%s\n"), Expr, File, Line, DescriptionString);
	}
}

class FErrorHistWriter
{
public:
	UE_NONCOPYABLE(FErrorHistWriter);

	inline FErrorHistWriter()
	{
		FailDebugMutex.Lock();
	}

	inline ~FErrorHistWriter()
	{
		TerminateWith(TEXT(""));
		FailDebugMutex.Unlock();
	}

	inline void TerminateWith(const TCHAR* Text)
	{
		const int32 TextLen = FCString::Strlen(Text);
		if (Index + TextLen >= Capacity)
		{
			Index = Capacity - TextLen - 1;
		}
		Append(Text);
		GErrorHist[Index] = TCHAR(0);
	}

	void Append(const TCHAR* Text)
	{
		const int32 TextLen = FCString::Strlen(Text);
		const int32 CopyLen = FPlatformMath::Min(TextLen + 1, Capacity - Index);
		if (CopyLen > 1)
		{
			FCString::Strncpy(GErrorHist + Index, Text, CopyLen);
			Index += CopyLen - 1;
		}
	}

	template <typename CharType>
	void Append(const CharType* Text)
	{
		if (const int32 TextLen = TCString<CharType>::Strlen(Text))
		{
			const int32 RequiredLen = FPlatformString::ConvertedLength<TCHAR>(Text, TextLen);
			if (Index + RequiredLen < Capacity)
			{
				FPlatformString::Convert(GErrorHist + Index, RequiredLen, Text, TextLen);
				Index += RequiredLen;
			}
		}
	}

	void AppendV(const TCHAR* Format, va_list Args)
	{
		const int32 Len = FPlatformString::GetVarArgs(GErrorHist + Index, Capacity - Index, Format, Args);
		Index = FMath::Clamp(Index, Index + Len, Capacity - 1);
	}

	void AppendV(const UTF8CHAR* Format, va_list Args)
	{
		if (const int32 Len = TCString<UTF8CHAR>::GetVarArgs(Utf8Scratch, UE_ARRAY_COUNT(Utf8Scratch), Format, Args); Len > 0)
		{
			Append(Utf8Scratch);
		}
	}

	UE_REWRITE void AppendV(const ANSICHAR* Format, va_list Args)
	{
		AppendV((const UTF8CHAR*)Format, Args);
	}

	void Appendf(const TCHAR* Format, ...)
	{
		va_list Args;
		va_start(Args, Format);
		AppendV(Format, Args);
		va_end(Args);
	}

private:
	int32 Index = 0;

	static constexpr int32 Capacity = UE_ARRAY_COUNT(GErrorHist);

	// Scratch space for formatting as UTF-8.
	// This is static to avoid stack overflows.
	// This is safe to be static because its use is guarded by FailDebugMutex.
	inline static UTF8CHAR Utf8Scratch[Capacity] = {};
};

/**
 * Prints error to the debug output and copies the error into the global error message.
 */
template <UE::CCharType FormatCharType>
FORCENOINLINE void StaticFailDebugV(
	const TCHAR* Error,
	const ANSICHAR* Expression,
	const ANSICHAR* File,
	int32 Line,
	bool bIsEnsure,
	void* ProgramCounter,
	const TCHAR* Message,
	const FormatCharType* DescriptionFormat,
	va_list DescriptionArgs)
{
	PrintScriptCallstack();

	// some platforms (Windows, Mac, Linux) generate this themselves by throwing an exception and capturing
	// the backtrace later on
	ANSICHAR StackTrace[4096] = "";
	if (FPlatformProperties::AllowsCallStackDumpDuringAssert() && !bIsEnsure)
	{
#if ASSERT_DISABLE_BACKTRACE_DUE_TO_ASAN
		FCStringAnsi::Snprintf(StackTrace, UE_ARRAY_COUNT(StackTrace), "(callstack disabled due to ASAN)");
#else
		FPlatformStackWalk::StackWalkAndDump(StackTrace, UE_ARRAY_COUNT(StackTrace), ProgramCounter);
#endif
	}

	FErrorHistWriter Writer;
	Writer.Append(Error);
	Writer.Append(Expression);
	Writer.Appendf(TEXT(FILE_LINE_DESC_ANSI "\n"), File ? File : "Unknown", Line);
	if (Message)
	{
		Writer.Append(Message);
		Writer.TerminateWith(TEXT("\n"));
	}
	if (DescriptionFormat)
	{
		Writer.AppendV(DescriptionFormat, DescriptionArgs);
		Writer.TerminateWith(TEXT("\n"));
	}

	FPlatformMisc::LowLevelOutputDebugString(GErrorHist);

	if (*StackTrace)
	{
		Writer.Append(StackTrace);
		Writer.TerminateWith(TEXT("\n"));
	}

	Writer.TerminateWith(TEXT("\r\n\r\n"));

	if (GError)
	{
		GError->SetErrorProgramCounter(ProgramCounter);
	}
}

FORCENOINLINE void VARARGS StaticFailDebug(const TCHAR* Error, const ANSICHAR* File, int32 Line, void* ProgramCounter, const TCHAR* Message)
{
	va_list DescriptionArgs{};
	StaticFailDebugV(Error, /*Expression*/ "", File, Line, /*bIsEnsure*/ false, ProgramCounter, Message, /*DescriptionFormat*/ (const TCHAR*)nullptr, DescriptionArgs);
}

FORCENOINLINE void VARARGS StaticFailDebugV(const TCHAR* Error, const ANSICHAR* File, int32 Line, void* ProgramCounter, const TCHAR* DescriptionFormat, va_list DescriptionArgs)
{
	StaticFailDebugV(Error, /*Expression*/ "", File, Line, /*bIsEnsure*/ false, ProgramCounter, /*Message*/ nullptr, DescriptionFormat, DescriptionArgs);
}

FORCENOINLINE void VARARGS StaticFailDebugV(const TCHAR* Error, const ANSICHAR* File, int32 Line, void* ProgramCounter, const ANSICHAR* DescriptionFormat, va_list DescriptionArgs)
{
	StaticFailDebugV(Error, /*Expression*/ "", File, Line, /*bIsEnsure*/ false, ProgramCounter, /*Message*/ nullptr, DescriptionFormat, DescriptionArgs);
}

FORCENOINLINE void VARARGS StaticFailDebugV(const TCHAR* Error, const ANSICHAR* File, int32 Line, void* ProgramCounter, const UTF8CHAR* DescriptionFormat, va_list DescriptionArgs)
{
	StaticFailDebugV(Error, /*Expression*/ "", File, Line, /*bIsEnsure*/ false, ProgramCounter, /*Message*/ nullptr, DescriptionFormat, DescriptionArgs);
}

/// track thread asserts
bool FDebug::HasAsserted()
{
	return bHasAsserted;
}

// track ensures
bool FDebug::IsEnsuring()
{
	return ActiveEnsureCount > 0;
}
SIZE_T FDebug::GetNumEnsureFailures()
{
	return NumEnsureFailures.Load();
}

void FDebug::LogFormattedMessageWithCallstack(const FName& InLogName, const ANSICHAR* File, int32 Line, const TCHAR* Heading, const TCHAR* Message, ELogVerbosity::Type Verbosity)
{
	FLogCategoryName LogName(InLogName);

	const bool bLowLevel = LogName == NAME_None;
	const bool bWriteUATMarkers = FParse::Param(FCommandLine::Get(), TEXT("CrashForUAT")) && FParse::Param(FCommandLine::Get(), TEXT("stdout")) && !bLowLevel;

	if (bWriteUATMarkers)
	{
		UE::Logging::Private::LogDebugMessageLine(File, Line, LogName, Verbosity, TEXT("begin: stack for UAT"));
	}

	if (bLowLevel)
	{
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("%s\n"), Heading);
	}
	else
	{
		UE::Logging::Private::LogDebugMessageLine(File, Line, LogName, Verbosity, Heading);
		UE::Logging::Private::LogDebugMessageLine(File, Line, LogName, Verbosity, TEXT(""));
	}

	for (const TCHAR* LineStart = Message;; )
	{
		TCHAR SingleLine[1024];

		// Find the end of the current line
		const TCHAR* LineEnd = LineStart;
		TCHAR* SingleLineWritePos = SingleLine;
		int32 SpaceRemaining = UE_ARRAY_COUNT(SingleLine) - 1;

		while (SpaceRemaining > 0 && *LineEnd != 0 && *LineEnd != '\r' && *LineEnd != '\n')
		{
			*SingleLineWritePos++ = *LineEnd++;
			--SpaceRemaining;
		}

		// cap it
		*SingleLineWritePos = TEXT('\0');

		// prefix function lines with [Callstack] for parsing tools
		const TCHAR* Prefix = (FCString::Strnicmp(LineStart, TEXT("0x"), 2) == 0) ? TEXT("[Callstack] ") : TEXT("");

		// if this is an address line, prefix it with [Callstack]
		if (bLowLevel)
		{
			FPlatformMisc::LowLevelOutputDebugStringf(TEXT("%s%s\n"), Prefix, SingleLine);
		}
		else
		{
			UE::Logging::Private::LogDebugMessageLine(File, Line, LogName, Verbosity, SingleLine, Prefix);
		}
		
		// Quit if this was the last line
		if (*LineEnd == 0)
		{
			break;
		}

		// Move to the next line
		LineStart = (LineEnd[0] == '\r' && LineEnd[1] == '\n') ? (LineEnd + 2) : (LineEnd + 1);
	}

	if (bWriteUATMarkers)
	{
		UE::Logging::Private::LogDebugMessageLine(File, Line, LogName, Verbosity, TEXT("end: stack for UAT"));
	}
}

#if DO_CHECK || DO_GUARD_SLOW || DO_ENSURE
//
// Failed assertion handler.
//warning: May be called at library startup time.
//

FORCENOINLINE void FDebug::LogAssertFailedMessageImpl(const ANSICHAR* Expr, const ANSICHAR* File, int32 Line, void* ProgramCounter, const TCHAR* Fmt, ...)
{
	va_list Args;
	va_start(Args, Fmt);
	LogAssertFailedMessageImplV(Expr, File, Line, ProgramCounter, Fmt, Args);
	va_end(Args);
}

void FDebug::LogAssertFailedMessageImplV(const ANSICHAR* Expr, const ANSICHAR* File, int32 Line, void* ProgramCounter, const TCHAR* Fmt, va_list Args)
{
	// Ignore this assert if we're already forcibly shutting down because of a critical error.
	if( !GIsCriticalError )
	{
		StaticFailDebugV(TEXT("Assertion failed: "), Expr, File, Line, /*bIsEnsure*/ false, ProgramCounter, /*Message*/ nullptr, Fmt, Args);
	}
}

thread_local TFunction<bool(const FEnsureHandlerArgs& Args)> EnsureHandler = nullptr;

TFunction<bool(const FEnsureHandlerArgs& Args)> SetEnsureHandler(TFunction<bool(const FEnsureHandlerArgs& Args)> Handler)
{
	TFunction<bool(const FEnsureHandlerArgs& Args)> OldHandler = EnsureHandler;
	EnsureHandler = MoveTemp(Handler);
	return OldHandler;
}

TFunction<bool(const FEnsureHandlerArgs& Args)> GetEnsureHandler()
{
	return EnsureHandler;
}

/**
 * Called when an 'ensure' assertion fails; gathers stack data and generates and error report.
 *
 * @param	Expr	Code expression ANSI string (#code)
 * @param	File	File name ANSI string (__FILE__)
 * @param	Line	Line number (__LINE__)
 * @param	Msg		Informative error message text
 * @param	NumStackFramesToIgnore	Number of stack frames to ignore in the callstack
 */
UE_AUTORTFM_ALWAYS_OPEN
FORCENOINLINE void FDebug::EnsureFailed(const ANSICHAR* Expr, const ANSICHAR* File, int32 Line, void* ProgramCounter, const TCHAR* Msg)
{
	if (UNLIKELY(EnsureHandler && EnsureHandler({ Expr, Msg })))
	{
		return;
	}
	FTempCommandLineScope TempCommandLine;

	// if time isn't ready yet, we better not continue
	if (FPlatformTime::GetSecondsPerCycle() == 0.0)
	{
		return;
	}
	
	++NumEnsureFailures;

#if STATS
	FString EnsureFailedPerfMessage = FString::Printf(TEXT("FDebug::EnsureFailed"));
	SCOPE_LOG_TIME_IN_SECONDS(*EnsureFailedPerfMessage, nullptr)
#endif

	// You can set bShouldCrash to true to cause a regular assertion to trigger (stopping program execution) when an ensure() error occurs
	const bool bShouldCrash = false;		// By default, don't crash on ensure()
	if( bShouldCrash )
	{
		// Just trigger a regular assertion which will crash via GError->Logf()
		FDebug::LogAssertFailedMessageImpl( Expr, File, Line, ProgramCounter, TEXT("%s"), Msg );
		return;
	}

	// Should we spin here?
	FPlatformAtomics::InterlockedIncrement(&ActiveEnsureCount);

	va_list DescriptionArgs{};
	StaticFailDebugV(TEXT("Ensure condition failed: "), Expr, File, Line, /*bIsEnsure*/ true, ProgramCounter, Msg, (const TCHAR*)nullptr, DescriptionArgs);

	// Is there a debugger attached?  If not we'll submit an error report.
	if (FPlatformMisc::IsDebuggerPresent() && !GAlwaysReportCrash)
	{
		const FString CombinedErrorMsg = FString::Printf(TEXT("Ensure condition failed: %hs" FILE_LINE_DESC_ANSI "\n%s\n"), Expr, File, Line, Msg);
		FCoreDelegates::OnEnsureFailed.Broadcast(Expr, File, Line, Msg, *CombinedErrorMsg);

#if !NO_LOGGING
		if (GEnsuresAreErrors)
		{
			UE_LOGFMT(LogOutputDevice, Error, "{CombinedErrorMsg}", CombinedErrorMsg);
		}
		else
		{
			UE_LOGFMT(LogOutputDevice, Warning, "{CombinedErrorMsg}", CombinedErrorMsg);
		}
#endif
	}
	else
	{
		// If we determine that we have not sent a report for this ensure yet, send the report below.
		bool bShouldSendNewReport = false;

		// Create a final string that we'll output to the log (and error history buffer)
		TStringBuilder<1024> ErrorMsg;
		ErrorMsg.Appendf(TEXT("Ensure condition failed: %hs " FILE_LINE_DESC_ANSI LINE_TERMINATOR_ANSI "%s" LINE_TERMINATOR_ANSI "Stack: " LINE_TERMINATOR_ANSI), Expr, File, Line, Msg);

		// No debugger attached, so generate a call stack and submit a crash report
		// Walk the stack and dump it to the allocated memory.
		const SIZE_T StackTraceSize = 65535;
		ANSICHAR* StackTrace = (ANSICHAR*)FMemory::SystemMalloc(StackTraceSize);
		if (StackTrace != NULL)
		{
			// Stop checking heartbeat for this thread (and stop the gamethread hitch detector if we're the game thread).
			// Ensure can take a lot of time (when stackwalking), so we don't want hitches/hangs firing.
			// These are no-ops on threads that didn't already have a heartbeat etc.
			FSlowHeartBeatScope SuspendHeartBeat;
			FDisableHitchDetectorScope SuspendGameThreadHitch;

			{
#if STATS
				FString StackWalkPerfMessage = FString::Printf(TEXT("FPlatformStackWalk::StackWalkAndDump"));
				SCOPE_LOG_TIME_IN_SECONDS(*StackWalkPerfMessage, nullptr)
#endif
				StackTrace[0] = 0;
#if ASSERT_DISABLE_BACKTRACE_DUE_TO_ASAN
				FCStringAnsi::Snprintf(StackTrace, StackTraceSize, "(callstack disabled due to ASAN)");
#else
				FPlatformStackWalk::StackWalkAndDumpEx(StackTrace, StackTraceSize, ProgramCounter, FGenericPlatformStackWalk::EStackWalkFlags::FlagsUsedWhenHandlingEnsure);
#endif
			}

			// Also append the stack trace
			ErrorMsg.Append(StackTrace);
			FMemory::SystemFree(StackTrace);

			// Dump the error and flush the log.
#if !NO_LOGGING
			FDebug::LogFormattedMessageWithCallstack(LogOutputDevice.GetCategoryName(), __FILE__, __LINE__, TEXT("=== Handled ensure: ==="), *ErrorMsg, (GEnsuresAreErrors ? ELogVerbosity::Error : ELogVerbosity::Warning));
#endif
			GLog->Flush();

			FCoreDelegates::OnEnsureFailed.Broadcast(Expr, File, Line, Msg, *ErrorMsg);

			// Trace the error
#if !NO_LOGGING
			if (GEnsuresAreErrors)
			{
				TRACE_LOG_MESSAGE(LogOutputDevice, Error, TEXT("%s"), *ErrorMsg);
			}
			else
			{
				TRACE_LOG_MESSAGE(LogOutputDevice, Warning, TEXT("%s"), *ErrorMsg);
			}
#endif
			if (FParse::Param(FCommandLine::Get(), TEXT("nocrashreports")))
			{
				return;
			}

			// Submit the error report to the server! (and display a balloon in the system tray)
			if (GEnsuresAreErrors)
			{
				// How many unique previous errors we should keep track of
				const uint32 MaxPreviousErrorsToTrack = 4;
				static uint32 StaticPreviousErrorCount = 0;
				if (StaticPreviousErrorCount < MaxPreviousErrorsToTrack)
				{
					// Check to see if we've already reported this error.  No point in blasting the server with
					// the same error over and over again in a single application session.
					bool bHasErrorAlreadyBeenReported = false;

					// Static: Array of previous unique error message CRCs
					static uint32 StaticPreviousErrorCRCs[MaxPreviousErrorsToTrack];

					// Compute CRC of error string.  Note that along with the call stack, this includes the message
					// string passed to the macro, so only truly redundant errors will go unreported.  Though it also
					// means you shouldn't pass loop counters to ensureMsgf(), otherwise failures may spam the server!
					const uint32 ErrorStrCRC = FCrc::StrCrc_DEPRECATED(*ErrorMsg);

					for (uint32 CurErrorIndex = 0; CurErrorIndex < StaticPreviousErrorCount; ++CurErrorIndex)
					{
						if (StaticPreviousErrorCRCs[CurErrorIndex] == ErrorStrCRC)
						{
							// Found it!  This is a redundant error message.
							bHasErrorAlreadyBeenReported = true;
							break;
						}
					}

					// Add the element to the list and bump the count
					StaticPreviousErrorCRCs[StaticPreviousErrorCount++] = ErrorStrCRC;

					if (!bHasErrorAlreadyBeenReported)
					{
#if STATS
						FString SubmitErrorReporterfMessage = FString::Printf(TEXT("SubmitErrorReport"));
						SCOPE_LOG_TIME_IN_SECONDS(*SubmitErrorReporterfMessage, nullptr)
#endif

						FCoreDelegates::OnHandleSystemEnsure.Broadcast();

						FPlatformMisc::SubmitErrorReport(*ErrorMsg, EErrorReportMode::Balloon);

						bShouldSendNewReport = true;
					}
				}
			}
		}
		else
		{
			const FString CombinedErrorMsg = FString::Printf(TEXT("Ensure condition failed: %hs" FILE_LINE_DESC_ANSI "\n%s\n"), Expr, File, Line, Msg);
			FCoreDelegates::OnEnsureFailed.Broadcast(Expr, File, Line, Msg, *CombinedErrorMsg);

			if (FParse::Param(FCommandLine::Get(), TEXT("nocrashreports")))
			{
				return;
			}

			// If we fail to generate the string to identify the crash we don't know if we should skip sending the report,
			// so we will just send the report anyway.
			bShouldSendNewReport = !!GEnsuresAreErrors;

			// Add message to log even without stacktrace. It is useful for testing fail on ensure.
#if !NO_LOGGING
			if (GEnsuresAreErrors)
			{
				UE_LOGFMT(LogOutputDevice, Error, "Ensure condition failed: {Expr} [File:{File}] [Line: {Line}]", Expr, File, Line);
			}
			else
			{
				UE_LOGFMT(LogOutputDevice, Warning, "Ensure condition failed: {Expr} [File:{File}] [Line: {Line}]", Expr, File, Line);
			}
#endif
		}

		if (bShouldSendNewReport)
		{
#if STATS
			FString SendNewReportMessage = FString::Printf(TEXT("SendNewReport"));
			SCOPE_LOG_TIME_IN_SECONDS(*SendNewReportMessage, nullptr)
#endif

#if PLATFORM_USE_REPORT_ENSURE
			UE::TUniqueLock Lock(FailDebugMutex);
			ReportEnsure(*ErrorMsg, ProgramCounter);

			GErrorHist[0] = TEXT('\0');
			GErrorExceptionDescription[0] = TEXT('\0');
#endif
		}
	}


	FPlatformAtomics::InterlockedDecrement(&ActiveEnsureCount);
}

bool FORCENOINLINE FDebug::CheckVerifyFailedImpl(
	const ANSICHAR* Expr,
	const ANSICHAR* File,
	int32 Line,
	void* ProgramCounter,
	const TCHAR* Format,
	...)
{
	va_list Args;

	va_start(Args, Format);
	FDebug::LogAssertFailedMessageImplV(Expr, File, Line, ProgramCounter, Format, Args);
	va_end(Args);

	if (GLog)
	{
		// Flushing the logs here increases the likelihood that recent messages will be written to the log file, stdout and the debugger console.
		// Without this, some of the recent messages may not be reported when debugger stops due to an assertion failure.
		GLog->Flush();
	}

	if (!FPlatformMisc::IsDebuggerPresent())
	{
		FPlatformMisc::PromptForRemoteDebugging(false);

		va_start(Args, Format);
		AssertFailedImplV(Expr, File, Line, ProgramCounter, Format, Args);
		va_end(Args);

		return false;
	}

#if UE_BUILD_SHIPPING
	return true;
#else
	return !GIgnoreDebugger;
#endif
}

UE_AUTORTFM_ALWAYS_OPEN
bool FORCENOINLINE FDebug::CheckVerifyFailedImpl2V(
	const ANSICHAR* Expr,
	const ANSICHAR* File,
	int32 Line,
	const TCHAR* Format,
	va_list Args)
{
	va_list Args2;
	va_copy(Args2, Args);
	FDebug::LogAssertFailedMessageImplV(Expr, File, Line, PLATFORM_RETURN_ADDRESS(), Format, Args2);
	va_end(Args2);

	if (GLog)
	{
		// Flushing the logs here increases the likelihood that recent messages will be written to the log file, stdout and the debugger console.
		// Without this, some of the recent messages may not be reported when debugger stops due to an assertion failure.
		GLog->Flush();
	}

	if (!FPlatformMisc::IsDebuggerPresent())
	{
		FPlatformMisc::PromptForRemoteDebugging(false);

		AssertFailedImplV(Expr, File, Line, PLATFORM_RETURN_ADDRESS(), Format, Args);

		return false;
	}

#if UE_BUILD_SHIPPING
	return true;
#else
	return !GIgnoreDebugger;
#endif
}

bool FORCENOINLINE FDebug::CheckVerifyFailedImpl2(
	const ANSICHAR* Expr,
	const ANSICHAR* File,
	int32 Line,
	const TCHAR* Format,
	...)
{
	va_list Args;
	va_start(Args, Format);
	bool bResult = CheckVerifyFailedImpl2V(Expr, File, Line, Format, Args);
	va_end(Args);

	return bResult;
}

#endif // DO_CHECK || DO_GUARD_SLOW || DO_ENSURE

void VARARGS FDebug::AssertFailed(const ANSICHAR* Expr, const ANSICHAR* File, int32 Line, const TCHAR* Format/* = TEXT("")*/, ...)
{
	va_list Args;
	va_start(Args, Format);
	AssertFailedImplV(Expr, File, Line, nullptr, Format, Args);
	va_end(Args);
}

void FDebug::AssertFailedV(const ANSICHAR* Expr, const ANSICHAR* File, int32 Line, const TCHAR* Format, va_list Args)
{
	AssertFailedImplV(Expr, File, Line, nullptr, Format, Args);
}

void FDebug::ProcessFatalError(void* ProgramCounter)
{
	// This is not perfect because another thread might crash and be handled before this assert
	// but this static variable will report the crash as an assert. Given complexity of a thread
	// aware solution, this should be good enough. If crash reports are obviously wrong we can
	// look into fixing this.
	bHasAsserted = true;

	GError->SetErrorProgramCounter(ProgramCounter);
	GError->Logf(TEXT("%s"), GErrorHist);
}

#if DO_CHECK || DO_GUARD_SLOW || DO_ENSURE
FORCENOINLINE bool VARARGS FDebug::OptionallyLogFormattedEnsureMessageReturningFalseImpl( bool bLog, const ANSICHAR* Expr, const ANSICHAR* File, int32 Line, void* ProgramCounter, const TCHAR* FormattedMsg, ... )
{
	va_list Args;
	va_start(Args, FormattedMsg);
	OptionallyLogFormattedEnsureMessageReturningFalseImpl(bLog, Expr, File, Line, ProgramCounter, FormattedMsg, Args);
	va_end(Args);

	return false;
}

namespace AssertionMacros_Private
{
	const int32 FormatBufferSize = 65535;
	TCHAR FormatBuffer[FormatBufferSize];
	UE::FWordMutex FormatMutex;
}

FORCENOINLINE bool FDebug::OptionallyLogFormattedEnsureMessageReturningFalseImpl(bool bLog, const ANSICHAR* Expr, const ANSICHAR* File, int32 Line, void* ProgramCounter, const TCHAR* FormattedMsg, va_list Args)
{
	if (bLog)
	{
		UE::TUniqueLock Lock(AssertionMacros_Private::FormatMutex);
		FCString::GetVarArgs(AssertionMacros_Private::FormatBuffer, AssertionMacros_Private::FormatBufferSize, FormattedMsg, Args);
		EnsureFailed(Expr, File, Line, ProgramCounter, AssertionMacros_Private::FormatBuffer);
	}

	return false;
}
#endif

void UE_COLD UE_DEBUG_SECTION VARARGS UE::Assert::Private::ProcessLowLevelFatalError(const ANSICHAR* File, int32 Line, const TCHAR* Format, ...)
{
	va_list Args;
	va_start(Args, Format);
	StaticFailDebugV(TEXT("LowLevelFatalError"), /*Expression*/ "", File, Line, /*bIsEnsure*/ false, PLATFORM_RETURN_ADDRESS(), /*Message*/ nullptr, Format, Args);
	va_end(Args);

	UE_DEBUG_BREAK_AND_PROMPT_FOR_REMOTE();
	FDebug::ProcessFatalError(PLATFORM_RETURN_ADDRESS());
}

void UE_COLD UE_DEBUG_SECTION VARARGS UE::Assert::Private::ProcessLowLevelFatalError(const ANSICHAR* File, int32 Line, const ANSICHAR* Format, ...)
{
	va_list Args;
	va_start(Args, Format);
	StaticFailDebugV(TEXT("LowLevelFatalError"), /*Expression*/ "", File, Line, /*bIsEnsure*/ false, PLATFORM_RETURN_ADDRESS(), /*Message*/ nullptr, (const UTF8CHAR*)Format, Args);
	va_end(Args);

	UE_DEBUG_BREAK_AND_PROMPT_FOR_REMOTE();
	FDebug::ProcessFatalError(PLATFORM_RETURN_ADDRESS());
}

void UE_COLD UE_DEBUG_SECTION VARARGS UE::Assert::Private::ProcessLowLevelFatalError(const ANSICHAR* File, int32 Line, const UTF8CHAR* Format, ...)
{
	va_list Args;
	va_start(Args, Format);
	StaticFailDebugV(TEXT("LowLevelFatalError"), /*Expression*/ "", File, Line, /*bIsEnsure*/ false, PLATFORM_RETURN_ADDRESS(), /*Message*/ nullptr, Format, Args);
	va_end(Args);

	UE_DEBUG_BREAK_AND_PROMPT_FOR_REMOTE();
	FDebug::ProcessFatalError(PLATFORM_RETURN_ADDRESS());
}

UE_AUTORTFM_ALWAYS_OPEN
void FDebug::DumpStackTraceToLog(const ELogVerbosity::Type LogVerbosity)
{
	DumpStackTraceToLog(TEXT("=== FDebug::DumpStackTrace(): ==="), LogVerbosity);
}

UE_AUTORTFM_ALWAYS_OPEN
FORCENOINLINE void FDebug::DumpStackTraceToLog(const TCHAR* Heading, const ELogVerbosity::Type LogVerbosity)
{
#if !NO_LOGGING
	// Walk the stack and dump it to the allocated memory.
	const SIZE_T StackTraceSize = 65535;
	ANSICHAR* StackTrace = (ANSICHAR*)FMemory::SystemMalloc(StackTraceSize);

	{
#if STATS
		FString StackWalkPerfMessage = FString::Printf(TEXT("FPlatformStackWalk::StackWalkAndDump"));
		SCOPE_LOG_TIME_IN_SECONDS(*StackWalkPerfMessage, nullptr)
#endif
		StackTrace[0] = 0;

#if ASSERT_DISABLE_BACKTRACE_DUE_TO_ASAN
		FCStringAnsi::Snprintf(StackTrace, StackTraceSize, "(callstack disabled due to ASAN)");
#else
		const int32 NumStackFramesToIgnore = 1;
		FPlatformStackWalk::StackWalkAndDumpEx(StackTrace, StackTraceSize, NumStackFramesToIgnore, FGenericPlatformStackWalk::EStackWalkFlags::FlagsUsedWhenHandlingEnsure);
#endif
	}

	// Dump the error and flush the log.
	// ELogVerbosity::Error to make sure it gets printed in log for conveniency.
	FDebug::LogFormattedMessageWithCallstack(LogOutputDevice.GetCategoryName(), __FILE__, __LINE__, Heading, ANSI_TO_TCHAR(StackTrace), LogVerbosity);
	GLog->Flush();
	FMemory::SystemFree(StackTrace);
#endif
}

#if DO_ENSURE && !USING_CODE_ANALYSIS
bool UE_COLD UE_DEBUG_SECTION VARARGS CheckVerifyImpl(const ANSICHAR* File, int32 Line, void* ProgramCounter, const ANSICHAR* Expr, const TCHAR* Format, va_list Args)
{
	FDebug::OptionallyLogFormattedEnsureMessageReturningFalse(true, Expr, File, Line, ProgramCounter, Format, Args);

	if (!FPlatformMisc::IsDebuggerPresent())
	{
		FPlatformMisc::PromptForRemoteDebugging(true);
		return false;
	}

#if UE_BUILD_SHIPPING
	return true;
#else
	return !GIgnoreDebugger;
#endif
}

UE_AUTORTFM_ALWAYS_OPEN
bool UE_COLD UE_DEBUG_SECTION UE::Assert::Private::CheckEnsureFailed(bool bAlways, std::atomic<uint8>& bExecuted, const ANSICHAR* File, int32 Line, const ANSICHAR* Expr)
{
	if (((bAlways && GEnsureAlwaysEnabled) || bExecuted.load(std::memory_order_relaxed) != GEnsureResetState) && FPlatformMisc::IsEnsureAllowed())
	{
		if (bExecuted.exchange(static_cast<uint8>(GEnsureResetState), std::memory_order_release) == GEnsureResetState && !bAlways)
		{
			return false;
		}

		va_list Args = {};
		const bool bResult = CheckVerifyImpl(File, Line, PLATFORM_RETURN_ADDRESS(), Expr, TEXT(""), Args);
		return bResult && GEnsureBreakEnabled;
	}

	return false;
}

// Check done separately so we don't evaluate the var args if we're not expecting to process the ensure
bool UE_COLD UE_DEBUG_SECTION UE::Assert::Private::CheckEnsureFailed(bool bAlways, const std::atomic<uint8>& bExecuted)
{
	return ((bAlways && GEnsureAlwaysEnabled) || bExecuted.load(std::memory_order_relaxed) != GEnsureResetState) && FPlatformMisc::IsEnsureAllowed();
}

UE_AUTORTFM_ALWAYS_OPEN
bool UE_COLD UE_DEBUG_SECTION EnsureFailedV(std::atomic<uint8>& bExecuted, const UE::Assert::Private::FStaticEnsureRecord* Ensure, void* ProgramCounter, va_list Args)
{
	if (bExecuted.exchange(static_cast<uint8>(GEnsureResetState), std::memory_order_release) == GEnsureResetState && !(Ensure->bAlways && GEnsureAlwaysEnabled))
	{
		return false;
	}

	const bool bResult = CheckVerifyImpl(Ensure->File, Ensure->Line, ProgramCounter, Ensure->Expression, Ensure->Format, Args);
	return bResult && GEnsureBreakEnabled;
}

bool UE_COLD UE_DEBUG_SECTION VARARGS UE::Assert::Private::EnsureFailed(std::atomic<uint8>& bExecuted, const FStaticEnsureRecord* Ensure, ...)
{
	va_list Args;
	va_start(Args, Ensure);
	const bool bResult = EnsureFailedV(bExecuted, Ensure, PLATFORM_RETURN_ADDRESS(), Args);
	va_end(Args);

	return bResult;
}
#endif

#undef FILE_LINE_DESC_ANSI
