// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AsyncTaskNotification.h"

#include "Misc/AssertionMacros.h"
#include "Misc/CoreAsyncTaskNotificationImpl.h"

FAsyncTaskNotification::FAsyncTaskNotification(const FAsyncTaskNotificationConfig& InConfig)
	: NotificationImpl(FAsyncTaskNotificationFactory::Get().InvokeFactory())
{
	check(NotificationImpl);
	NotificationImpl->Initialize(InConfig);
}

FAsyncTaskNotification::~FAsyncTaskNotification()
{
}

FAsyncTaskNotification::FAsyncTaskNotification(FAsyncTaskNotification&& InOther)
	: NotificationImpl(InOther.NotificationImpl)
{
	InOther.NotificationImpl.Reset();
}

FAsyncTaskNotification& FAsyncTaskNotification::operator=(FAsyncTaskNotification&& InOther)
{
	if (this != &InOther)
	{
		NotificationImpl.Reset();
		NotificationImpl = InOther.NotificationImpl;
		InOther.NotificationImpl.Reset();
	}
	return *this;
}

void FAsyncTaskNotification::SetTitleText(const FText& InTitleText, const bool bClearProgressText)
{
	NotificationImpl->SetTitleText(InTitleText, bClearProgressText);
}

void FAsyncTaskNotification::SetProgressText(const FText& InProgressText)
{
	NotificationImpl->SetProgressText(InProgressText);
}

void FAsyncTaskNotification::SetPromptText(const FText& InPromptText)
{
	NotificationImpl->SetPromptText(InPromptText);
}

void FAsyncTaskNotification::SetHyperlink(const FSimpleDelegate& InHyperlink, const FText& InHyperlinkText)
{
	NotificationImpl->SetHyperlink(InHyperlink, InHyperlinkText);
}

void FAsyncTaskNotification::SetComplete(const bool bSuccess)
{
	NotificationImpl->SetComplete(bSuccess);
}

void FAsyncTaskNotification::SetComplete(const FText& InTitleText, const FText& InProgressText, const bool bSuccess)
{
	NotificationImpl->SetComplete(InTitleText, InProgressText, bSuccess);
}

void FAsyncTaskNotification::SetNotificationState(const FAsyncNotificationStateData& InState)
{
	NotificationImpl->SetNotificationState(InState);
}

void FAsyncTaskNotification::SetCanCancel(const TAttribute<bool>& InCanCancel)
{
	NotificationImpl->SetCanCancel(InCanCancel);
}

void FAsyncTaskNotification::SetKeepOpenOnSuccess(const TAttribute<bool>& InKeepOpenOnSuccess)
{
	NotificationImpl->SetKeepOpenOnSuccess(InKeepOpenOnSuccess);
}

void FAsyncTaskNotification::SetKeepOpenOnFailure(const TAttribute<bool>& InKeepOpenOnFailure)
{
	NotificationImpl->SetKeepOpenOnFailure(InKeepOpenOnFailure);
}

EAsyncTaskNotificationPromptAction FAsyncTaskNotification::GetPromptAction() const
{
	return NotificationImpl->GetPromptAction();
}
