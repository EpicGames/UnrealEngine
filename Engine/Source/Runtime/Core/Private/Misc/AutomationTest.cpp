// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"

#include <inttypes.h>

#include "AutoRTFM.h"
#include "Algo/Copy.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/PlatformStackWalk.h"
#include "HAL/ThreadHeartBeat.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Regex.h"
#include "Logging/StructuredLog.h"
#include "Logging/TokenizedMessage.h"
#include "Misc/App.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/FileHelper.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Misc/Paths.h"
#include "Misc/ScopedSlowTask.h"
#include "Misc/ScopeRWLock.h"
#include "Misc/TextFilterExpressionEvaluator.h"
#include "Modules/ModuleManager.h"

DEFINE_LOG_CATEGORY(LogLatentCommands);
DEFINE_LOG_CATEGORY(LogAutomationTestFramework);
DEFINE_LOG_CATEGORY_STATIC(LogAutomationTestStateTrace, Log, All);
DEFINE_LOG_CATEGORY_STATIC(LogAutomationTest, Warning, All);

namespace AutomationTest
{
	static bool bCaptureLogEvents = true;
	static FAutoConsoleVariableRef CVarAutomationCaptureLogEvents(
		TEXT("Automation.CaptureLogEvents"),
		bCaptureLogEvents,
		TEXT("Consider warning/error log events during a test as impacting the test itself"));

	static bool bSkipStackWalk = false;
	static FAutoConsoleVariableRef CVarAutomationSkipStackWalk(
		TEXT("Automation.SkipStackWalk"),
		bSkipStackWalk,
		TEXT("Whether to skip any stack issues that the automation test framework triggers"));

	static bool bLogBPTestMetadata = false;
	static FAutoConsoleVariableRef CVarAutomationLogBPTestMetadata(
		TEXT("Automation.LogBPTestMetadata"),
		bLogBPTestMetadata,
		TEXT("Whether to output blueprint functional test metadata to the log when test is running"));

	static bool bLogTestStateTrace = false;
	static FAutoConsoleVariableRef CVarAutomationLogTestStateTrace(
		TEXT("Automation.LogTestStateTrace"),
		bLogTestStateTrace,
		TEXT("Whether to enable or disable logging of test state trace"));

	static bool bEnableStereoTestVariants = false;
	static FAutoConsoleVariableRef CVarAutomationEnableStereoTestVariants(
		TEXT("Automation.EnableStereoTestVariants"),
		bEnableStereoTestVariants,
		TEXT("Whether to enable stereo test variants for screenshot functional tests"));

	static bool bLightweightStereoTestVariants = true;
	static FAutoConsoleVariableRef CVarAutomationLightweightStereoTestVariants(
		TEXT("Automation.LightweightStereoTestVariants"),
		bLightweightStereoTestVariants,
		TEXT("Whether to skip variants when the baseline test fails, and skip saving screenshots for successful variants"));

	FString TestTagGlobalFilter;
	static FAutoConsoleVariableRef CVarAutomationTestTagGlobalFilter(
		TEXT("Automation.TestTagGlobalFilter"),
		TestTagGlobalFilter,
		TEXT("Only include tests marked with Tags matching this filter string, using the Advanced Search Syntax"));

	// The method prepares the filename and LineNumber to be placed in the form that could be extracted by SAutomationWindow widget if it is additionally eclosed into []
	// The result format is filename(line)
	static FString CreateFileLineDescription(const FString& Filename, const int32 LineNumber)
	{
		FString Result;

		if (!Filename.IsEmpty() && LineNumber > 0)
		{
			Result += Filename;
			Result += TEXT("(");
			Result += FString::FromInt(LineNumber);
			Result += TEXT(")");
		}

		return Result;
	}

	/*
		Determine the level that a log item should be written to the automation log based on the properties of the current test.
		only Display/Warning/Error are supported in the automation log so anything with NoLogging/Log will not be shown
	*/
	static ELogVerbosity::Type GetAutomationLogLevel(ELogVerbosity::Type LogVerbosity, FName LogCategory, FAutomationTestBase* CurrentTest)
	{
		ELogVerbosity::Type EffectiveVerbosity = LogVerbosity;

		static FTransactionallySafeCriticalSection ActionCS;
		static FAutomationTestBase* LastTest = nullptr;

		if (AutomationTest::bCaptureLogEvents == false)
		{
			return ELogVerbosity::NoLogging;
		}

		{
			UE::TScopeLock Lock(ActionCS);
			if (CurrentTest != LastTest)
			{
				FAutomationTestBase::SuppressedLogCategories.Empty();
				FAutomationTestBase::LoadDefaultLogSettings();
				LastTest = CurrentTest;
			}
		}

		if (CurrentTest)
		{
			if (CurrentTest->SuppressLogs() || CurrentTest->GetSuppressedLogCategories().Contains(LogCategory.ToString()))
			{
				EffectiveVerbosity = ELogVerbosity::NoLogging;
			}
			else
			{
				if (EffectiveVerbosity == ELogVerbosity::Warning)
				{
					if (CurrentTest->SuppressLogWarnings())
					{
						EffectiveVerbosity = ELogVerbosity::NoLogging;
					}
					else if (CurrentTest->ElevateLogWarningsToErrors())
					{
						EffectiveVerbosity = ELogVerbosity::Error;
					}
				}

				if (EffectiveVerbosity == ELogVerbosity::Error)
				{
					if (CurrentTest->SuppressLogErrors())
					{
						EffectiveVerbosity = ELogVerbosity::NoLogging;
					}
				}
			}
		}

		return EffectiveVerbosity;
	}

	static inline bool GetTestClassAndParameters(const FString& InTest, FString* OutTestClass, FString* OutTestParameters)
	{
		return InTest.Split(TEXT(" "), OutTestClass, OutTestParameters, ESearchCase::CaseSensitive);
	}
};

FAutomationTestBase::FAutomationTestBase( const FString& InName, const bool bInComplexTask )
	: bComplexTask( bInComplexTask )
{
	LLM_SCOPE_BYNAME(TEXT("AutomationTest/Framework"));
	TestName = InName.Replace(TEXT(" "), TEXT("_"));
	// Register the newly created automation test into the automation testing framework
	const bool bRegistered = FAutomationTestFramework::Get().RegisterAutomationTest( TestName, this );
	if (!bRegistered)
	{
		UE_LOGF(LogAutomationTest, Warning, "Failed to register test with the name '%ls'. Test with the same name is already registered and will not be overridden.", *TestName);
	}
}

/** Destructor */
FAutomationTestBase::~FAutomationTestBase() 
{ 
	// Unregister the automation test from the automation testing framework
	FAutomationTestFramework::Get().UnregisterAutomationTest( TestName );
}

bool FAutomationTestBase::bSuppressLogWarnings = false;
bool FAutomationTestBase::bSuppressLogErrors = false;
bool FAutomationTestBase::bElevateLogWarningsToErrors = false;
TArray<FString> FAutomationTestBase::SuppressedLogCategories;

CORE_API const TMap<FString, EAutomationTestFlags>& EAutomationTestFlags_GetTestFlagsMap()
{
	LLM_SCOPE_BYNAME(TEXT("AutomationTest/Framework"));
	/** String to EAutomationTestFlags map */
	static const TMap<FString, EAutomationTestFlags> FlagsMap = {
		{ TEXT("EditorContext"),          EAutomationTestFlags::EditorContext},
		{ TEXT("ClientContext"),          EAutomationTestFlags::ClientContext},
		{ TEXT("ServerContext"),          EAutomationTestFlags::ServerContext},
		{ TEXT("CommandletContext"),      EAutomationTestFlags::CommandletContext},
		{ TEXT("ProgramContext"),         EAutomationTestFlags::ProgramContext},
		{ TEXT("SupportsAutoRTFM"),       EAutomationTestFlags::SupportsAutoRTFM},
		{ TEXT("ApplicationContextMask"), EAutomationTestFlags_ApplicationContextMask},
		{ TEXT("NonNullRHI"),             EAutomationTestFlags::NonNullRHI},
		{ TEXT("RequiresUser"),           EAutomationTestFlags::RequiresUser},
		{ TEXT("FeatureMask"),            EAutomationTestFlags_FeatureMask},
		{ TEXT("Disabled"),               EAutomationTestFlags::Disabled},
		{ TEXT("CriticalPriority"),       EAutomationTestFlags::CriticalPriority},
		{ TEXT("HighPriority"),           EAutomationTestFlags::HighPriority},
		{ TEXT("HighPriorityAndAbove"),   EAutomationTestFlags_HighPriorityAndAbove},
		{ TEXT("MediumPriority"),         EAutomationTestFlags::MediumPriority},
		{ TEXT("MediumPriorityAndAbove"), EAutomationTestFlags_MediumPriorityAndAbove},
		{ TEXT("LowPriority"),            EAutomationTestFlags::LowPriority},
		{ TEXT("PriorityMask"),           EAutomationTestFlags_PriorityMask},
		{ TEXT("SmokeFilter"),            EAutomationTestFlags::SmokeFilter},
		{ TEXT("EngineFilter"),           EAutomationTestFlags::EngineFilter},
		{ TEXT("ProductFilter"),          EAutomationTestFlags::ProductFilter},
		{ TEXT("PerfFilter"),             EAutomationTestFlags::PerfFilter},
		{ TEXT("StressFilter"),           EAutomationTestFlags::StressFilter},
		{ TEXT("NegativeFilter"),         EAutomationTestFlags::NegativeFilter},
		{ TEXT("FilterMask"),             EAutomationTestFlags_FilterMask}
	};
	return FlagsMap;
};

void FAutomationTestFramework::FAutomationTestOutputDevice::Serialize( const TCHAR* V, ELogVerbosity::Type Verbosity, const class FName& Category )
{
	const int32 STACK_OFFSET = 8;//FMsg::Logf_InternalImpl
	// TODO would be nice to search for the first stack frame that isn't in output device or other logging files, would be more robust.

	if (!IsRunningCommandlet() && (Verbosity == ELogVerbosity::SetColor))
	{
		return;
	}

	// Ensure there's a valid unit test associated with the context
	FAutomationTestBase* const LocalCurTest = CurTest.load(std::memory_order_relaxed);
	if (LocalCurTest)
	{
		bool CaptureLog = !LocalCurTest->SuppressLogs()
			&& (Verbosity == ELogVerbosity::Error || Verbosity == ELogVerbosity::Warning || Verbosity == ELogVerbosity::Display)
			&& LocalCurTest->ShouldCaptureLogCategory(Category);

		if (CaptureLog)
		{
			ELogVerbosity::Type EffectiveVerbosity = AutomationTest::GetAutomationLogLevel(Verbosity, Category, LocalCurTest);
			if (EffectiveVerbosity != ELogVerbosity::NoLogging)
			{
				FString FormattedMsg = FString::Printf(TEXT("%s: %s"), *Category.ToString(), V);

				FAutomationEvent Event(EAutomationEventType::Info, FormattedMsg, TEXT("log"));
				// Errors
				if (EffectiveVerbosity == ELogVerbosity::Error)
				{
					Event.Type = EAutomationEventType::Error;
				}
				// Warnings
				else if (EffectiveVerbosity == ELogVerbosity::Warning)
				{
					Event.Type = EAutomationEventType::Warning;
				}
				LocalCurTest->AddEvent(Event, STACK_OFFSET);
			}
		}
		// Log...etc
		else
		{
			// IMPORTANT NOTE: This code will never be called in a build with NO_LOGGING defined, which means pretty much
			// any Test or Shipping config build.  If you're trying to use the automation test framework for performance
			// data capture in a Test config, you'll want to call the AddAnalyticsItemToCurrentTest() function instead of
			// using this log interception stuff.

			FString LogString = FString(V);
			FString AnalyticsString = "AUTOMATIONANALYTICS";
			if (LogString.StartsWith(*AnalyticsString))
			{
				//Remove "analytics" from the string
				LogString.RightInline(LogString.Len() - (AnalyticsString.Len() + 1), EAllowShrinking::No);

				LocalCurTest->AddAnalyticsItem(LogString);
			}
			//else
			//{
			//	LocalCurTest->AddInfo(LogString, STACK_OFFSET);
			//}
		}
	}
}

void FAutomationTestFramework::FAutomationTestMessageFilter::Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const FName& Category)
{
	Serialize(V, Verbosity, Category, -1.0);
}

void FAutomationTestFramework::FAutomationTestMessageFilter::Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const FName& Category, double Time)
{
	// Prevent null dereference if logging happens in async tasks while changing DestinationContext
	FFeedbackContext* const LocalDestinationContext = DestinationContext.load(std::memory_order_relaxed);
	FAutomationTestBase* const LocalCurTest = CurTest.load(std::memory_order_relaxed);
	if (LocalDestinationContext)
	{
		if (LocalCurTest && LocalCurTest->IsExpectedMessage(FString(V), Verbosity))
		{
			Verbosity = ELogVerbosity::Verbose;
		}
		{
			UE::TScopeLock CriticalSection(ActionCS);
			LocalDestinationContext->Serialize(V, Verbosity, Category, Time);
		}
	}
}

void FAutomationTestFramework::FAutomationTestMessageFilter::SerializeRecord(const UE::FLogRecord& Record)
{
	// Prevent null dereference if logging happens in async tasks while changing DestinationContext
	FFeedbackContext* const LocalDestinationContext = DestinationContext.load(std::memory_order_relaxed);
	FAutomationTestBase* const LocalCurTest = CurTest.load(std::memory_order_relaxed);
	if (LocalDestinationContext)
	{
		UE::FLogRecord LocalRecord = Record;
		const ELogVerbosity::Type Verbosity = LocalRecord.GetVerbosity();
		if ((Verbosity == ELogVerbosity::Warning) || (Verbosity == ELogVerbosity::Error))
		{
			TStringBuilder<512> Line;
			Record.FormatMessageTo(Line);
			if (LocalCurTest && LocalCurTest->IsExpectedMessage(FString(Line), ELogVerbosity::Warning))
			{
				LocalRecord.SetVerbosity(ELogVerbosity::Verbose);
			}
		}
		{
			UE::TScopeLock CriticalSection(ActionCS);
			LocalDestinationContext->SerializeRecord(LocalRecord);
		}
	}
}

FAutomationTestFramework& FAutomationTestFramework::Get()
{
	static FAutomationTestFramework Framework;
	return Framework;
}

FString FAutomationTestFramework::GetUserAutomationDirectory() const
{
	const FString DefaultAutomationSubFolder = "Unreal Automation";
	return FString(FPlatformProcess::UserDir()) + DefaultAutomationSubFolder;
}

bool FAutomationTestFramework::NeedSkipStackWalk()
{
	return AutomationTest::bSkipStackWalk;
}


bool FAutomationTestFramework::NeedLogBPTestMetadata()
{
	return AutomationTest::bLogBPTestMetadata;
}

bool FAutomationTestFramework::NeedPerformStereoTestVariants()
{
	return AutomationTest::bEnableStereoTestVariants;
}

bool FAutomationTestFramework::NeedUseLightweightStereoTestVariants()
{
	return AutomationTest::bLightweightStereoTestVariants;
}

bool FAutomationTestFramework::RegisterAutomationTest( const FString& InTestNameToRegister, FAutomationTestBase* InTestToRegister )
{
	const bool bAlreadyRegistered = AutomationTestClassNameToInstanceMap.Contains( InTestNameToRegister );
	if ( !bAlreadyRegistered )
	{
		LLM_SCOPE_BYNAME(TEXT("AutomationTest/Framework"));
		AutomationTestClassNameToInstanceMap.Add( InTestNameToRegister, InTestToRegister );
	}
	return !bAlreadyRegistered;
}

bool FAutomationTestFramework::UnregisterAutomationTest( const FString& InTestNameToUnregister )
{
	const bool bRegistered = AutomationTestClassNameToInstanceMap.Contains( InTestNameToUnregister );
	if ( bRegistered )
	{
		AutomationTestClassNameToInstanceMap.Remove( InTestNameToUnregister );
	}
	return bRegistered;
}

bool FAutomationTestFramework::RegisterAutomationTestTags( const FString& InTestNameToRegister, const FString& InTestTagsToRegister, bool InImmutable)
{
	TArray<FString> TagsToRegister;
	if (InImmutable)
	{
		const bool bAlreadyRegistered = TestFullNameToTagDataMap.Contains(InTestNameToRegister);
		if (!bAlreadyRegistered)
		{
			LLM_SCOPE_BYNAME(TEXT("AutomationTest/Framework"));
			TestFullNameToTagDataMap.Add(InTestNameToRegister, InTestTagsToRegister);

			InTestTagsToRegister.Replace(TEXT("]"), TEXT("")).ParseIntoArray(TagsToRegister, TEXT("["), true);
			AllExistingTags.Append(TagsToRegister);

			if (!ImmutableTags.Contains(InTestNameToRegister))
			{
				ImmutableTags.Add(InTestNameToRegister, TSet<FString>());
			}
			ImmutableTags[InTestNameToRegister].Append(TagsToRegister);
		}
		return !bAlreadyRegistered;
	}
	else // Merge with existing tags, keep immutable tags
	{
		LLM_SCOPE_BYNAME(TEXT("AutomationTest/Framework"));
		InTestTagsToRegister.Replace(TEXT("]"), TEXT("")).ParseIntoArray(TagsToRegister, TEXT("["), true);
		AllExistingTags.Append(TagsToRegister);

		TSet<FString> MergedTags =
			TSet<FString>(TagsToRegister)
			.Union(TSet<FString>(
				GetTagsForAutomationTestAsArray(InTestTagsToRegister))
				.Intersect(ImmutableTags.Contains(InTestNameToRegister) ? ImmutableTags[InTestNameToRegister] : TSet<FString>()));

		TestFullNameToTagDataMap.Remove(InTestNameToRegister);
		TestFullNameToTagDataMap.Add(InTestNameToRegister,
			FString::JoinBy(MergedTags, TEXT(""),
			[](const FString& InTag)
			{
				return FString::Printf(TEXT("[%s]"), *InTag);
			}));
		return true;
	}
}

bool FAutomationTestFramework::UnregisterAutomationTestTags(const FString& InTestNameToUnregister)
{
	const bool bRegistered = TestFullNameToTagDataMap.Contains(InTestNameToUnregister);
	if (bRegistered)
	{
		TestFullNameToTagDataMap.Remove(InTestNameToUnregister);
	}
	return bRegistered;
}

bool FAutomationTestFramework::RegisterComplexAutomationTestTags(const FAutomationTestBase* InTest, const FString& InBeautifiedTestName, const FString& InTestTagsToRegister)
{
	FString FullTestName = InTest->GetBeautifiedTestName().AppendChar('.').Append(InBeautifiedTestName);
	return RegisterAutomationTestTags(FullTestName, InTestTagsToRegister);
}

TSet<FString> FAutomationTestFramework::GetAllExistingTags()
{
	return AllExistingTags;
}

bool FAutomationTestFramework::IsTagImmutable(const FString& InTestName, const FString& InTag) const
{
	return ImmutableTags.Contains(InTestName) && ImmutableTags[InTestName].Contains(InTag);
}

FString FAutomationTestFramework::GetTagsForAutomationTest(const FString& InTestName)
{
	FString * FindResult = TestFullNameToTagDataMap.Find(InTestName);
	if (FindResult)
	{
		return *FindResult;
	}
	return FString();
}

TArray<FString> FAutomationTestFramework::GetTagsForAutomationTestAsArray(const FString& InTestName)
{
	TArray<FString> Tags;
	FString* FindResult = TestFullNameToTagDataMap.Find(InTestName);
	if (FindResult)
	{
		FindResult->Replace(TEXT("]"), TEXT("[")).ParseIntoArray(Tags, TEXT("["));
	}
	return Tags;
}

void FAutomationTestFramework::EnqueueLatentCommand(TSharedPtr<IAutomationLatentCommand> NewCommand)
{
	//ensure latent commands are never used within smoke tests - will only catch when smokes are exclusively requested
	check((RequestedTestFilter & EAutomationTestFlags_FilterMask) != EAutomationTestFlags::SmokeFilter);

	//ensure we are currently "running a test"
	check(GIsAutomationTesting);

	LatentCommands.Enqueue(NewCommand);
}

void FAutomationTestFramework::EnqueueNetworkCommand(TSharedPtr<IAutomationNetworkCommand> NewCommand)
{
	//ensure latent commands are never used within smoke tests
	check((RequestedTestFilter & EAutomationTestFlags_FilterMask) != EAutomationTestFlags::SmokeFilter);

	//ensure we are currently "running a test"
	check(GIsAutomationTesting);

	NetworkCommands.Enqueue(NewCommand);
}

bool FAutomationTestFramework::ContainsTest( const FString& InTestName ) const
{
	return AutomationTestClassNameToInstanceMap.Contains( InTestName );
}

static double SumDurations(const TMap<FString, FAutomationTestExecutionInfo>& Executions)
{
	double Sum = 0;
	for (const TPair<FString, FAutomationTestExecutionInfo>& Execution : Executions)
	{
		Sum += Execution.Value.Duration;
	}
	return Sum;
}

static const TCHAR* FindSlowestTest(const TMap<FString, FAutomationTestExecutionInfo>& Executions, double& OutMaxDuration)
{
	check(Executions.Num() > 0);

	const TCHAR* OutName = nullptr;
	OutMaxDuration = 0;
	for (const TPair<FString, FAutomationTestExecutionInfo>& Execution : Executions)
	{
		if (OutMaxDuration <= Execution.Value.Duration)
		{
			OutMaxDuration = Execution.Value.Duration;
			OutName = *Execution.Key;
		}
	}

	return OutName;
}

bool FAutomationTestFramework::RunSmokeTests()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FAutomationTestFramework::RunSmokeTests);

	bool bAllSuccessful = true;

	//so extra log spam isn't generated
	TGuardValue<EAutomationTestFlags> GuardRequestedTestFilter(RequestedTestFilter, EAutomationTestFlags::SmokeFilter);
	
	// Skip running on cooked platforms like mobile
	//@todo - better determination of whether to run than requires cooked data
	// Ensure there isn't another slow task in progress when trying to run unit tests
	constexpr bool bRequiresCookedData = FPlatformProperties::RequiresCookedData();
	if ((!bRequiresCookedData & !GIsSlowTask & !GIsPlayInEditorWorld & !FPlatformProperties::IsProgram() & !IsRunningCommandlet()) || bForceSmokeTests)
	{
		TArray<FAutomationTestInfo> TestInfo;

		GetValidTestNames( TestInfo );

		if ( TestInfo.Num() > 0 )
		{
			// Output the results of running the automation tests
			TMap<FString, FAutomationTestExecutionInfo> OutExecutionInfoMap;

			// Run each valid test
			FScopedSlowTask SlowTask((float)TestInfo.Num());

			// We disable capturing the stack when running smoke tests, it adds too much overhead to do it at startup.
			FAutomationTestFramework::Get().SetCaptureStack(false);

			for ( int TestIndex = 0; TestIndex < TestInfo.Num(); ++TestIndex )
			{
				SlowTask.EnterProgressFrame(1);
				if (!!(TestInfo[TestIndex].GetTestFlags() & EAutomationTestFlags::SmokeFilter))
				{
					FString TestCommand = TestInfo[TestIndex].GetTestName();
					FAutomationTestExecutionInfo& CurExecutionInfo = OutExecutionInfoMap.Add( TestCommand, FAutomationTestExecutionInfo() );
					
					int32 RoleIndex = 0;  //always default to "local" role index.  Only used for multi-participant tests
					StartTestByName( TestCommand, RoleIndex );
					const bool CurTestSuccessful = GIsAutomationTesting && StopTest(CurExecutionInfo);

					bAllSuccessful = bAllSuccessful && CurTestSuccessful;
				}
			}

			FAutomationTestFramework::Get().SetCaptureStack(true);

#if !UE_BUILD_DEBUG
			const double TotalDuration = SumDurations(OutExecutionInfoMap);
			if (bAllSuccessful && !FPlatformMisc::IsDebuggerPresent() && TotalDuration > 2.0)
			{
				//force a failure if a smoke test takes too long
				double SlowestTestDuration = 0;
				const TCHAR* SlowestTestName = FindSlowestTest(OutExecutionInfoMap, /* out */ SlowestTestDuration);
				UE_LOGF(LogAutomationTest, Warning, "Smoke tests took >2s to run (%.2fs). '%ls' took %dms. "
					"SmokeFilter tier tests should take less than 1ms. Please optimize or move '%ls' to a slower tier than SmokeFilter.", 
					TotalDuration, SlowestTestName, static_cast<int32>(1000*SlowestTestDuration), SlowestTestName);
			}
#endif

			FAutomationTestFramework::DumpAutomationTestExecutionInfo( OutExecutionInfoMap );
		}
	}
	else if (IsRunningCommandlet() || bRequiresCookedData)
	{
		UE_LOGF( LogAutomationTest, Log, "Skipping unit tests for the cooked build and commandlet." );
	}
	else if (!FPlatformProperties::IsProgram())
	{
		UE_LOGF(LogAutomationTest, Error, "Skipping unit tests." );
		bAllSuccessful = false;
	}

	return bAllSuccessful;
}

void FAutomationTestFramework::ResetTests()
{
	bool bEnsureExists = false;
	bool bDeleteEntireTree = true;
	//make sure all transient files are deleted successfully
	IFileManager::Get().DeleteDirectory(*FPaths::AutomationTransientDir(), bEnsureExists, bDeleteEntireTree);
}

void FAutomationTestFramework::StartTestByName( const FString& InTestToRun, const int32 InRoleIndex, const FString& InFullTestPath )
{
	if (GIsAutomationTesting)
	{
		while(!LatentCommands.IsEmpty())
		{
			TSharedPtr<IAutomationLatentCommand> TempCommand;
			LatentCommands.Dequeue(TempCommand);
		}
		while(!NetworkCommands.IsEmpty())
		{
			TSharedPtr<IAutomationNetworkCommand> TempCommand;
			NetworkCommands.Dequeue(TempCommand);
		}
		FAutomationTestExecutionInfo TempExecutionInfo;
		StopTest(TempExecutionInfo);
	}

	FString TestName;
	FString Params;
	if (!AutomationTest::GetTestClassAndParameters(InTestToRun, &TestName, &Params))
	{
		TestName = InTestToRun;
	}
	FString TestPath = InFullTestPath.IsEmpty() ? InTestToRun : InFullTestPath;

	NetworkRoleIndex = InRoleIndex;

	// Ensure there isn't another slow task in progress when trying to run unit tests
	if ( !GIsSlowTask && !GIsPlayInEditorWorld )
	{
		// Ensure the test exists in the framework and is valid to run
		if ( ContainsTest( TestName ) )
		{
			// Make any setting changes that have to occur to support unit testing
			PrepForAutomationTests();

			InternalStartTest( InTestToRun, TestPath);
		}
		else
		{
			UE_LOGF(LogAutomationTest, Error, "Test %ls does not exist and could not be run.", *TestPath);
		}
	}
	else
	{
		UE_LOGF(LogAutomationTest, Error, "Test %ls is too slow and could not be run.", *TestPath);
	}
}

bool FAutomationTestFramework::StopTest( FAutomationTestExecutionInfo& OutExecutionInfo )
{
	check(GIsAutomationTesting);
	
	bool bSuccessful = InternalStopTest(OutExecutionInfo);

	// Restore any changed settings now that unit testing has completed
	ConcludeAutomationTests();

	return bSuccessful;
}


bool FAutomationTestFramework::ExecuteLatentCommands()
{
	check(GIsAutomationTesting);

	bool bHadAnyLatentCommands = !LatentCommands.IsEmpty();
	while (!LatentCommands.IsEmpty())
	{
		//get the next command to execute
		TSharedPtr<IAutomationLatentCommand> NextCommand;
		LatentCommands.Peek(NextCommand);

		bool bComplete = NextCommand->InternalUpdate();
		if (bComplete)
		{
			TSharedPtr<IAutomationLatentCommand>* TailCommand = LatentCommands.Peek();
			if (TailCommand != nullptr && NextCommand == *TailCommand)
			{
				//all done. remove the tail
				LatentCommands.Pop();
			}
			else
			{
				UE_LOGF(LogAutomationTest, Verbose, "Tail of latent command queue is not removed, because last completed automation latent command is not corresponding.");
			}
		}
		else
		{
			break;
		}
	}
	//need more processing on the next frame
	if (bHadAnyLatentCommands)
	{
		return false;
	}

	return true;
}

bool FAutomationTestFramework::ExecuteNetworkCommands()
{
	check(GIsAutomationTesting);
	bool bHadAnyNetworkCommands = !NetworkCommands.IsEmpty();

	if( bHadAnyNetworkCommands )
	{
		// Get the next command to execute
		TSharedPtr<IAutomationNetworkCommand> NextCommand;
		NetworkCommands.Dequeue(NextCommand);
		if (NextCommand->GetRoleIndex() == NetworkRoleIndex)
		{
			NextCommand->Run();
		}
	}

	return !bHadAnyNetworkCommands;
}

void FAutomationTestFramework::DequeueAllCommands()
{
	while (!LatentCommands.IsEmpty())
	{
		TSharedPtr<IAutomationLatentCommand> TempCommand;
		LatentCommands.Dequeue(TempCommand);
	}
	while (!NetworkCommands.IsEmpty())
	{
		TSharedPtr<IAutomationNetworkCommand> TempCommand;
		NetworkCommands.Dequeue(TempCommand);
	}
}

void FAutomationTestFramework::LoadTestModules( )
{
	const bool bRunningEditor = GIsEditor && !IsRunningCommandlet();

	bool bRunningSmokeTests = ((RequestedTestFilter & EAutomationTestFlags_FilterMask) == EAutomationTestFlags::SmokeFilter);
	if( !bRunningSmokeTests )
	{
		TArray<FString> EngineTestModules;
		GConfig->GetArray( TEXT("/Script/Engine.AutomationTestSettings"), TEXT("EngineTestModules"), EngineTestModules, GEngineIni);
		//Load any engine level modules.
		for( int32 EngineModuleId = 0; EngineModuleId < EngineTestModules.Num(); ++EngineModuleId)
		{
			const FName ModuleName = FName(*EngineTestModules[EngineModuleId]);
			//Make sure that there is a name available.  This can happen if a name is left blank in the Engine.ini
			if (ModuleName == NAME_None || ModuleName == TEXT("None"))
			{
				UE_LOGF(LogAutomationTest, Warning, "The automation test module ('%ls') doesn't have a valid name.", *ModuleName.ToString());
				continue;
			}
			if (!FModuleManager::Get().IsModuleLoaded(ModuleName))
			{
				UE_LOGF(LogAutomationTest, Log, "Loading automation test module: '%ls'.", *ModuleName.ToString());
				FModuleManager::Get().LoadModule(ModuleName);
			}
		}
		//Load any editor modules.
		if( bRunningEditor )
		{
			TArray<FString> EditorTestModules;
			GConfig->GetArray( TEXT("/Script/Engine.AutomationTestSettings"), TEXT("EditorTestModules"), EditorTestModules, GEngineIni);
			for( int32 EditorModuleId = 0; EditorModuleId < EditorTestModules.Num(); ++EditorModuleId )
			{
				const FName ModuleName = FName(*EditorTestModules[EditorModuleId]);
				//Make sure that there is a name available.  This can happen if a name is left blank in the Engine.ini
				if (ModuleName == NAME_None || ModuleName == TEXT("None"))
				{
					UE_LOGF(LogAutomationTest, Warning, "The automation test module ('%ls') doesn't have a valid name.", *ModuleName.ToString());
					continue;
				}
				if (!FModuleManager::Get().IsModuleLoaded(ModuleName))
				{
					UE_LOGF(LogAutomationTest, Log, "Loading automation test module: '%ls'.", *ModuleName.ToString());
					FModuleManager::Get().LoadModule(ModuleName);
				}
			}
		}
	}
}

void FAutomationTestFramework::GetValidTestNames( TArray<FAutomationTestInfo>& TestInfo ) const
{
	LLM_SCOPE_BYNAME(TEXT("AutomationTest/Framework"));
	TestInfo.Empty();

	// Determine required application type (Editor, Game, or Commandlet)
	const bool bRunningCommandlet = IsRunningCommandlet();
	const bool bRunningEditor = GIsEditor && !bRunningCommandlet;
	const bool bRunningClient = !GIsEditor && !IsRunningDedicatedServer() && !FPlatformProperties::IsProgram();
	const bool bRunningServer = !GIsEditor && !IsRunningClientOnly() && !FPlatformProperties::IsProgram();
	const bool bRunningProgram = !GIsEditor && FPlatformProperties::IsProgram();

	//application flags
	EAutomationTestFlags ApplicationSupportFlags = EAutomationTestFlags::None;
	if ( bRunningEditor )
	{
		ApplicationSupportFlags |= EAutomationTestFlags::EditorContext;
	}
	if ( bRunningClient )
	{
		ApplicationSupportFlags |= EAutomationTestFlags::ClientContext;
	}
	if ( bRunningServer )
	{
		ApplicationSupportFlags |= EAutomationTestFlags::ServerContext;
	}
	if ( bRunningCommandlet )
	{
		ApplicationSupportFlags |= EAutomationTestFlags::CommandletContext;
	}
	if ( bRunningProgram )
	{
		ApplicationSupportFlags |= EAutomationTestFlags::ProgramContext;
	}

	//Feature support - assume valid RHI until told otherwise
	EAutomationTestFlags FeatureSupportFlags = EAutomationTestFlags_FeatureMask;
	// @todo: Handle this correctly. GIsUsingNullRHI is defined at Engine-level, so it can't be used directly here in Core.
	// For now, assume Null RHI is only used for commandlets, servers, and when the command line specifies to use it.
	if (FPlatformProperties::SupportsWindowedMode())
	{
		bool bUsingNullRHI = FParse::Param( FCommandLine::Get(), TEXT("nullrhi") ) || IsRunningCommandlet() || IsRunningDedicatedServer();
		if (bUsingNullRHI)
		{
			FeatureSupportFlags &= (~EAutomationTestFlags::NonNullRHI);
		}
	}
	if (FApp::IsUnattended())
	{
		FeatureSupportFlags &= (~EAutomationTestFlags::RequiresUser);
	}

	for ( TMap<FString, FAutomationTestBase*>::TConstIterator TestIter( AutomationTestClassNameToInstanceMap ); TestIter; ++TestIter )
	{
		const FAutomationTestBase* CurTest = TestIter.Value();
		check( CurTest );

		EAutomationTestFlags CurTestFlags = CurTest->GetTestFlags();

		//filter out full tests when running smoke tests
		const bool bPassesFilterRequirement = !!(CurTestFlags & RequestedTestFilter);

		//Application Tests
		EAutomationTestFlags CurTestApplicationFlags = (CurTestFlags & EAutomationTestFlags_ApplicationContextMask);
		const bool bPassesApplicationRequirements = !CurTestApplicationFlags || !!(CurTestApplicationFlags & ApplicationSupportFlags);
		
		//Feature Tests
		EAutomationTestFlags CurTestFeatureFlags = (CurTestFlags & EAutomationTestFlags_FeatureMask);
		const bool bPassesFeatureRequirements = !CurTestFeatureFlags || !!(CurTestFeatureFlags & FeatureSupportFlags);

		const bool bEnabled = !(CurTestFlags & EAutomationTestFlags::Disabled);
		if (bEnabled && bPassesApplicationRequirements && bPassesFeatureRequirements && bPassesFilterRequirement)
		{
			// Make sure people are not writing complex tests that take forever to return the names of the tests
			// otherwise the session frontend locks up when looking at your local tests.
			const double GenerateTestNamesStartTime = FPlatformTime::Seconds();

			TArray<FAutomationTestInfo> TestsToAdd;
			CurTest->GenerateTestNames(TestsToAdd);
		
			const double GenerateTestNamesEndTime = FPlatformTime::Seconds();
			const double TimeForGetTests = GenerateTestNamesEndTime - GenerateTestNamesStartTime;
			if (TimeForGetTests > 10.0)
			{
				// Emit a warning if GetTests() takes too long.
				UE_LOGF(LogAutomationTest, Warning, "Automation Test '%ls' took > 10 seconds to return from GetTests(...): %.2fs", *CurTest->GetTestName(), (float)TimeForGetTests);
			}

			TestInfo.Append(TestsToAdd);
		}
	}
}

bool FAutomationTestFramework::TagsMatchPattern(const FString& Tags, const FString& TagPattern) const
{
	check(TagFilter);
	TagFilter->SetFilterText(FText::FromString(TagPattern));
	return TagFilter->TestTextFilter(FBasicStringFilterExpressionContext(Tags));
}

void FAutomationTestFramework::GetTestFullNamesMatchingTagPattern(TArray<FString>& OutTestNames, const FString& TagPattern) const
{
	LLM_SCOPE_BYNAME(TEXT("AutomationTest/Framework"));
	OutTestNames.Empty();

	for (TMap<FString, FString>::TConstIterator TestIter(TestFullNameToTagDataMap); TestIter; ++TestIter)
	{
		const FString CurTags = TestIter.Value();
		if (TagsMatchPattern(CurTags, TagPattern))
		{
			OutTestNames.Add(TestIter.Key());
		}
	}
}

void FAutomationTestFramework::GetPossibleRestrictedPaths(const FString& BasePath, const TArray<FString>& RestrictedFolderNames, TArray<FString>& OutRestrictedFolders) const
{
	FString PotentialPath;
	for (const FString& RestrictedFolderName : RestrictedFolderNames)
	{
		PotentialPath = BasePath / RestrictedFolderName;
		if (IFileManager::Get().DirectoryExists(*PotentialPath))
		{
			OutRestrictedFolders.Add(PotentialPath);
			GetPossibleRestrictedPaths(PotentialPath, RestrictedFolderNames, OutRestrictedFolders);
		}
	}
}

void FAutomationTestFramework::LoadTestTagMappings()
{
	static bool bTagMappingFilesLoaded = false;

	if (!bTagMappingFilesLoaded)
	{
		LoadTestTagMappings(FPaths::ConvertRelativePathToFull(FPaths::ProjectDir()), false);
		LoadTestTagMappings(FPaths::ConvertRelativePathToFull(FPaths::EngineDir()), true);
		
		bTagMappingFilesLoaded = true;
	}
}

void FAutomationTestFramework::LoadTestTagMappings(const FString& BasePath, bool RestrictedPathsOnly)
{
	TArray<FString> RestrictedFolderNames = FPaths::GetRestrictedFolderNames();
	RestrictedFolderNames.Append({ TEXT("Platforms"), TEXT("Restricted") });

	TArray<FString> AllPaths;
	if (!RestrictedPathsOnly)
	{
		AllPaths.Add(BasePath);
	}

	GetPossibleRestrictedPaths(BasePath, RestrictedFolderNames, AllPaths);

	FString TagsSettingsFilePath;
	TArray<FString> TestTagMappings;

	for (const FString& Path : AllPaths)
	{
		TagsSettingsFilePath = Path / TEXT("Config") / TEXT("Tests") / TEXT("Tags.ini");

		if (IFileManager::Get().FileExists(*TagsSettingsFilePath))
		{
			UE_LOGF(LogAutomationTest, Log, "Loading tags from %ls.", *TagsSettingsFilePath);
			TArray<FString> Lines;
			if (FFileHelper::LoadFileToStringArray(Lines, *TagsSettingsFilePath))
			{
				const FRegexPattern FileNameToTagPattern(TEXT("(.*)=(\\[.+\\])+"), ERegexPatternFlags::CaseInsensitive);
				for (FString Line : Lines)
				{
					if (Line.StartsWith(TEXT(";")))
					{
						continue;
					}
					FRegexMatcher Matcher(FileNameToTagPattern, Line);
					if (Matcher.FindNext())
					{
						FString TestName = Matcher.GetCaptureGroup(1);
						FString TestTags = Matcher.GetCaptureGroup(2);
						if (!TestName.IsEmpty() && !TestTags.IsEmpty())
						{
							RegisterAutomationTestTags(TestName, TestTags, false);
						}
					}
				}
			}
			else
			{
				UE_LOGF(LogAutomationTest, Error, "Could not load mapping file %ls", *TagsSettingsFilePath);
			}
		}
	}
}

void FAutomationTestFramework::SaveTestTagMappings(const TArray<FString>& TestFullNames, const TArray<FString>& TestFilePaths, const FBeforeTagMappingConfigSaved& BeforeConfigSaved, const FAfterTagMappingConfigSaved& AfterConfigSaved) const
{
	check(TestFullNames.Num() == TestFilePaths.Num());

	const FString ProjectDirPath = FPaths::ConvertRelativePathToFull(*FPaths::ProjectDir());
	const FString EngineDirPath = FPaths::ConvertRelativePathToFull(FPaths::EngineDir());

	TArray<FString> RestrictedFolderNames = FPaths::GetRestrictedFolderNames();
	RestrictedFolderNames.Append({ TEXT("Platforms"), TEXT("Restricted") });

	TArray<FString> AllPathsProject;
	// Search for both restricted and non-restricted paths for tests living inside a project
	GetPossibleRestrictedPaths(ProjectDirPath, RestrictedFolderNames, AllPathsProject);
	TArray<FString> AllRestrictedPathsEngine;
	// Search strictly for restricted paths for tests living under Engine/
	GetPossibleRestrictedPaths(EngineDirPath, RestrictedFolderNames, AllRestrictedPathsEngine);

	AllPathsProject.Add(ProjectDirPath);
	AllPathsProject.Sort([](const FString& A, const FString& B) { return A.Len() > B.Len(); });

	AllRestrictedPathsEngine.Sort([](const FString& A, const FString& B) { return A.Len() > B.Len(); });

	FString TestSourcePath;
	FString TagsSettingsFilePath;
	TMap<FString, FConfigFile> AffectedTagConfigPaths;
	for (int i = 0; i < TestFilePaths.Num(); i++)
	{
		TagsSettingsFilePath.Empty();
		TestSourcePath = TestFilePaths[i];
		// Pick longest project path with restricted names that is a suffix of the test path, or the base project path if not in restricted path
		for (int j = 0 ; j < AllPathsProject.Num(); j++)
		{
			if (FPaths::IsUnderDirectory(TestSourcePath, AllPathsProject[j]))
			{
				TagsSettingsFilePath = AllPathsProject[j] / TEXT("Config") / TEXT("Tests") / TEXT("Tags.ini");
				break;
			}
		}

		if (TagsSettingsFilePath.IsEmpty())
		{
			for (int j = 0; j < AllRestrictedPathsEngine.Num(); j++)
			{
				if (FPaths::IsUnderDirectory(TestSourcePath, AllRestrictedPathsEngine[j]))
				{
					TagsSettingsFilePath = AllRestrictedPathsEngine[j] / TEXT("Config") / TEXT("Tests") / TEXT("Tags.ini");
					break;
				}
			}
		}

		// Tests that aren't in any restricted path under Engine/ will end up in NotForLicensees
		if (TagsSettingsFilePath.IsEmpty() && FPaths::IsUnderDirectory(TestSourcePath, EngineDirPath))
		{
			TagsSettingsFilePath = EngineDirPath / TEXT("Restricted") / TEXT("NotForLicensees") / TEXT("Config") / TEXT("Tests") / TEXT("Tags.ini");
		}

		if (TagsSettingsFilePath.IsEmpty())
		{
			UE_LOGF(LogAutomationTest, Error, "Test neither in project path or a restricted engine path, cannot be mapped. Path: %ls", *TestSourcePath);
			continue;
		}

		if (!AffectedTagConfigPaths.Contains(TagsSettingsFilePath))
		{
			AffectedTagConfigPaths.Add(TagsSettingsFilePath, FConfigFile());

			// Removal of readonly flag for Combine and Write to work properly
			FPlatformFileManager::Get().GetPlatformFile().SetReadOnly(*TagsSettingsFilePath, false);

			BeforeConfigSaved.ExecuteIfBound(TagsSettingsFilePath);

			// If file exists, import lines first before updating with SetInSection
			if (IFileManager::Get().FileExists(*TagsSettingsFilePath))
			{
				AffectedTagConfigPaths[TagsSettingsFilePath].Combine(TagsSettingsFilePath);
			}
		}

		AffectedTagConfigPaths[TagsSettingsFilePath].SetInSection(TEXT("TestTagMapping"), FName(TestFullNames[i]), *TestFullNameToTagDataMap.Find(TestFullNames[i]));
	}

	TSet<FString> AllConfigsPaths;
	AffectedTagConfigPaths.GetKeys(AllConfigsPaths);
	for (const FString& ConfigPath : AllConfigsPaths)
	{
		UE_LOGF(LogAutomationTestFramework, Log, "Saving tags to %ls.", *ConfigPath);
		AffectedTagConfigPaths[ConfigPath].Write(ConfigPath);
		AfterConfigSaved.ExecuteIfBound(ConfigPath);

	}
}

bool FAutomationTestFramework::ShouldTestContent(const FString& Path) const
{
	static TArray<FString> TestLevelFolders;
	if ( TestLevelFolders.Num() == 0 )
	{
		GConfig->GetArray( TEXT("/Script/Engine.AutomationTestSettings"), TEXT("TestLevelFolders"), TestLevelFolders, GEngineIni);
	}

	bool bMatchingDirectory = false;
	for ( const FString& Folder : TestLevelFolders )
	{
		const FString PatternToCheck = FString::Printf(TEXT("/%s/"), *Folder);
		if ( Path.Contains(*PatternToCheck) )
		{
			bMatchingDirectory = true;
		}
	}
	if (bMatchingDirectory)
	{
		return true;
	}

	const FString RelativePath = FPaths::ConvertRelativePathToFull(Path);
	const FString DevelopersPath = FPaths::ConvertRelativePathToFull(FPaths::GameDevelopersDir());
	return bDeveloperDirectoryIncluded || !RelativePath.StartsWith(DevelopersPath);
}

void FAutomationTestFramework::SetDeveloperDirectoryIncluded(const bool bInDeveloperDirectoryIncluded)
{
	bDeveloperDirectoryIncluded = bInDeveloperDirectoryIncluded;
}

void FAutomationTestFramework::SetRequestedTestFilter(const EAutomationTestFlags InRequestedTestFlags)
{
	RequestedTestFilter = InRequestedTestFlags;
}

FOnTestScreenshotCaptured& FAutomationTestFramework::OnScreenshotCaptured()
{
	return TestScreenshotCapturedDelegate;
}

FOnTestScreenshotAndTraceCaptured& FAutomationTestFramework::OnScreenshotAndTraceCaptured()
{
	return TestScreenshotAndTraceCapturedDelegate;
}

FOnTestSectionEvent& FAutomationTestFramework::GetOnEnteringTestSection(const FString& Section)
{
	if (!OnEnteringTestSectionEvent.Contains(Section))
	{
		OnEnteringTestSectionEvent.Emplace(Section);
	}

	return *OnEnteringTestSectionEvent.Find(Section);
}

void FAutomationTestFramework::TriggerOnEnteringTestSection(const FString& Section) const
{
	if (const FOnTestSectionEvent* Delegate = OnEnteringTestSectionEvent.Find(Section))
	{
		Delegate->Broadcast(Section);
	}
}

bool FAutomationTestFramework::IsAnyOnEnteringTestSectionBound() const
{
	if (!OnEnteringTestSectionEvent.IsEmpty())
	{
		for (auto& SectionPair : OnEnteringTestSectionEvent)
		{
			if (SectionPair.Value.IsBound())
			{
				return true;
			}
		}
	}

	return false;
}

FOnTestSectionEvent& FAutomationTestFramework::GetOnLeavingTestSection(const FString& Section)
{
	if (!OnLeavingTestSectionEvent.Contains(Section))
	{
		OnLeavingTestSectionEvent.Emplace(Section);
	}

	return *OnLeavingTestSectionEvent.Find(Section);
}

void FAutomationTestFramework::TriggerOnLeavingTestSection(const FString& Section) const
{
	if (const FOnTestSectionEvent* Delegate = OnLeavingTestSectionEvent.Find(Section))
	{
		Delegate->Broadcast(Section);
	}
}

bool FAutomationTestFramework::IsAnyOnLeavingTestSectionBound() const
{
	if (!OnLeavingTestSectionEvent.IsEmpty())
	{
		for (auto& SectionPair : OnLeavingTestSectionEvent)
		{
			if (SectionPair.Value.IsBound())
			{
				return true;
			}
		}
	}

	return false;
}

void FAutomationTestFramework::PrepForAutomationTests()
{
	check(!GIsAutomationTesting);

	// Fire off callback signifying that unit testing is about to begin. This allows
	// other systems to prepare themselves as necessary without the unit testing framework having to know
	// about them.
	PreTestingEvent.Broadcast();

	OriginalGWarn = GWarn;
	AutomationTestMessageFilter.SetDestinationContext(GWarn);
	GWarn = &AutomationTestMessageFilter;
	GLog->AddOutputDevice(&AutomationTestOutputDevice);

	// Mark that unit testing has begun
	GIsAutomationTesting = true;
}

void FAutomationTestFramework::ConcludeAutomationTests()
{
	check(GIsAutomationTesting);
	
	// Mark that unit testing is over
	GIsAutomationTesting = false;

	GLog->RemoveOutputDevice(&AutomationTestOutputDevice);
	GWarn = OriginalGWarn;
	AutomationTestMessageFilter.SetDestinationContext(nullptr);
	OriginalGWarn = nullptr;

	// Fire off callback signifying that unit testing has concluded.
	PostTestingEvent.Broadcast();
}

/**
 * Helper method to dump the contents of the provided test name to execution info map to the provided feedback context
 *
 * @param	InContext		Context to dump the execution info to
 * @param	InInfoToDump	Execution info that should be dumped to the provided feedback context
 */
void FAutomationTestFramework::DumpAutomationTestExecutionInfo( const TMap<FString, FAutomationTestExecutionInfo>& InInfoToDump )
{
	const FString SuccessMessage = NSLOCTEXT("UnrealEd", "AutomationTest_Success", "Success").ToString();
	const FString FailMessage = NSLOCTEXT("UnrealEd", "AutomationTest_Fail", "Fail").ToString();
	for ( TMap<FString, FAutomationTestExecutionInfo>::TConstIterator MapIter(InInfoToDump); MapIter; ++MapIter )
	{
		const FString& CurTestName = MapIter.Key();
		const FAutomationTestExecutionInfo& CurExecutionInfo = MapIter.Value();

		UE_LOGF(LogAutomationTest, Log, "%ls: %ls (%.2fms)", *CurTestName, CurExecutionInfo.bSuccessful ? *SuccessMessage : *FailMessage, 1000.0 * CurExecutionInfo.Duration);

		for ( const FAutomationExecutionEntry& Entry : CurExecutionInfo.GetEntries() )
		{
			switch (Entry.Event.Type )
			{
				case EAutomationEventType::Info:
					UE_LOGF(LogAutomationTest, Display, "%ls", *Entry.Event.Message);
					break;
				case EAutomationEventType::Warning:
					UE_LOGF(LogAutomationTest, Warning, "%ls", *Entry.Event.Message);
					break;
				case EAutomationEventType::Error:
					UE_LOGF(LogAutomationTest, Error, "%ls", *Entry.Event.Message);
					break;
			}
		}
	}
}

void FAutomationTestFramework::InternalStartTest( const FString& InTestToRun, const FString& InFullTestPath)
{
	Parameters.Empty();
	CurrentTestFullPath.Empty();

	FString TestName;
	if (!AutomationTest::GetTestClassAndParameters(InTestToRun, &TestName, &Parameters))
	{
		TestName = InTestToRun;
	}

	if ( ContainsTest( TestName ) )
	{
		CurrentTest = *( AutomationTestClassNameToInstanceMap.Find( TestName ) );
		check( CurrentTest );

		// Clear any execution info from the test in case it has been run before
		CurrentTest->ClearExecutionInfo();

		// Associate the test that is about to be run with the special unit test output device and feedback context
		AutomationTestOutputDevice.SetCurrentAutomationTest(CurrentTest);
		AutomationTestMessageFilter.SetCurrentAutomationTest(CurrentTest);

		StartTime = FPlatformTime::Seconds();

		CurrentTest->SetTestContext(Parameters);
		CurrentTestFullPath = InFullTestPath;

		// If not a smoke test, log the test has started.
		EAutomationTestFlags NonSmokeTestFlags = (EAutomationTestFlags_FilterMask & (~EAutomationTestFlags::SmokeFilter));
		if (!!(RequestedTestFilter & NonSmokeTestFlags))
		{
			if (AutomationTest::bLogTestStateTrace)
			{
				UE_LOGF(LogAutomationTestStateTrace, Log, "Test is about to start. Name={%ls}", *CurrentTestFullPath);
			}
			UE_LOGF(LogAutomationTest, Log, "%ls %ls is starting at %f", *CurrentTest->GetBeautifiedTestName(), *Parameters, StartTime);
		}

		OnTestStartEvent.Broadcast(CurrentTest);

		{
			TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("AutomationTest %s"), *CurrentTest->GetBeautifiedTestName()));
			// Run the test!
			bTestSuccessful = CurrentTest->RunTest(Parameters);
		}

#if UE_AUTORTFM
		const bool bTestWithAutoRTFM = !!(CurrentTest->GetTestFlags() & EAutomationTestFlags::SupportsAutoRTFM);
		if (bTestSuccessful && bTestWithAutoRTFM && AutoRTFM::ForTheRuntime::IsAutoRTFMRuntimeEnabled())
		{
			{
				TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("AutomationTest %s - AutoRTFM Commit"), *CurrentTest->GetBeautifiedTestName()));
				AutoRTFM::ETransactionResult Outcome = AutoRTFM::Transact([&]
				{
					if (!CurrentTest->RunTest(Parameters))
					{
						bTestSuccessful = false;
					}
				});
				if (Outcome != AutoRTFM::ETransactionResult::Committed)
				{
					UE_LOGF(LogAutomationTest, Warning, "%ls %ls: Test failed while committing an AutoRTFM transaction!", *CurrentTest->GetBeautifiedTestName(), *Parameters);
					bTestSuccessful = false;
				}
			}
			{
				TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("AutomationTest %s - AutoRTFM Abort"), *CurrentTest->GetBeautifiedTestName()));
				AutoRTFM::ETransactionResult Outcome = AutoRTFM::Transact([&]
				{
					if (!CurrentTest->RunTest(Parameters))
					{
						UE_AUTORTFM_ONABORT(this)
						{
							bTestSuccessful = false;
						};
					}
					AutoRTFM::AbortTransaction();
				});
				if (Outcome != AutoRTFM::ETransactionResult::AbortedByRequest)
				{
					UE_LOGF(LogAutomationTest, Warning, "%ls %ls: Test failed while aborting an AutoRTFM transaction!", *CurrentTest->GetBeautifiedTestName(), *Parameters);
					bTestSuccessful = false;
				}
			}
			if (!bTestSuccessful)
			{
				UE_LOGF(LogAutomationTest, Warning, "%ls %ls: Test passes normally, but fails under AutoRTFM!", *CurrentTest->GetBeautifiedTestName(), *Parameters);
			}
		}
#endif

	}
}

bool FAutomationTestFramework::InternalStopTest(FAutomationTestExecutionInfo& OutExecutionInfo)
{
	check(GIsAutomationTesting);
	check(LatentCommands.IsEmpty());

	// Determine if the test was successful based on three criteria:
	// 1) Did the test itself report success?
	// 2) Did any errors occur and were logged by the feedback context during execution?++----
	// 3) Did we meet any errors that were expected with this test
	bTestSuccessful = bTestSuccessful && !CurrentTest->HasAnyErrors() && CurrentTest->HasMetExpectedMessages();

	{
		UE::TWriteScopeLock Lock(CurrentTest->ActionCS);
		CurrentTest->ExpectedMessages.Empty();
	}

	// Set the success state of the test based on the above criteria
	CurrentTest->InternalSetSuccessState(bTestSuccessful);

	OnTestEndEvent.Broadcast(CurrentTest);

	double EndTime = FPlatformTime::Seconds();
	double TimeForTest = static_cast<float>(EndTime - StartTime);
	EAutomationTestFlags NonSmokeTestFlags = (EAutomationTestFlags_FilterMask & (~EAutomationTestFlags::SmokeFilter));
	if (!!(RequestedTestFilter & NonSmokeTestFlags))
	{
		UE_LOGF(LogAutomationTest, Log, "%ls %ls ran in %f", *CurrentTest->GetBeautifiedTestName(), *Parameters, TimeForTest);
		if (AutomationTest::bLogTestStateTrace)
		{
			UE_LOGF(LogAutomationTestStateTrace, Log, "Test has stopped execution. Name={%ls}", *CurrentTestFullPath);
		}
	}

	// Fill out the provided execution info with the info from the test
	CurrentTest->GetExecutionInfo( OutExecutionInfo );

	// Save off timing for the test
	OutExecutionInfo.Duration = TimeForTest;

	// Disassociate the test from the output device and feedback context
	AutomationTestOutputDevice.SetCurrentAutomationTest(nullptr);
	AutomationTestMessageFilter.SetCurrentAutomationTest(nullptr);

	// Release pointers to now-invalid data
	CurrentTest = NULL;

	return bTestSuccessful;
}

bool FAutomationTestFramework::CanRunTestInEnvironment(const FString& InTestToRun, FString* OutReason, bool* OutWarn) const
{
	FString TestClassName;
	FString TestParameters;
	if (!AutomationTest::GetTestClassAndParameters(InTestToRun, &TestClassName, &TestParameters))
	{
		TestClassName = InTestToRun;
	}

	if (!ContainsTest(TestClassName))
	{
		if (OutReason != nullptr)
		{
			*OutReason = FString::Printf(TEXT("Could not find test %s in test registry."), *TestClassName);
		}

		if (OutWarn != nullptr)
		{
			*OutWarn = true;
		}

		return false;
	}

	const FAutomationTestBase* const Test = *(AutomationTestClassNameToInstanceMap.Find(TestClassName));

	if (nullptr == Test)
	{
		return false;
	}

	if (!Test->CanRunInEnvironment(TestParameters, OutReason, OutWarn))
	{
		if (nullptr != OutReason)
		{
			if (OutReason->IsEmpty())
			{
				*OutReason = TEXT("unknown reason");
			}

			*OutReason += TEXT(" [code]");
			FString Filename = Test->GetTestSourceFileName();
			FPaths::MakePlatformFilename(Filename);
			const FString FileLineDescription = AutomationTest::CreateFileLineDescription(Filename, Test->GetTestSourceFileLine());
			if (!FileLineDescription.IsEmpty())
			{
				*OutReason += TEXT(" [");
				*OutReason += FileLineDescription;
				*OutReason += TEXT("]");
			}
		}
		
		return false;
	}

	return true;
}

void FAutomationTestFramework::AddAnalyticsItemToCurrentTest( const FString& AnalyticsItem )
{
	if( CurrentTest != nullptr )
	{
		CurrentTest->AddAnalyticsItem( AnalyticsItem );
	}
	else
	{
		UE_LOGF( LogAutomationTest, Warning, "AddAnalyticsItemToCurrentTest() called when no automation test was actively running!" );
	}
}

void FAutomationTestFramework::NotifyScreenshotComparisonComplete(const FAutomationScreenshotCompareResults& CompareResults)
{
	OnScreenshotCompared.Broadcast(CompareResults);
}

void FAutomationTestFramework::NotifyScreenshotComparisonReport(const FAutomationScreenshotCompareResults& CompareResults)
{
	OnScreenshotComparisonReport.Broadcast(CompareResults);
}

void FAutomationTestFramework::NotifyTestDataRetrieved(bool bWasNew, const FString& JsonData)
{
	OnTestDataRetrieved.Broadcast(bWasNew, JsonData);
}

void FAutomationTestFramework::NotifyPerformanceDataRetrieved(bool bSuccess, const FString& ErrorMessage)
{
	OnPerformanceDataRetrieved.Broadcast(bSuccess, ErrorMessage);
}

void FAutomationTestFramework::NotifyScreenshotTakenAndCompared()
{
	OnScreenshotTakenAndCompared.Broadcast();
}

FAutomationTestFramework::FAutomationTestFramework()
	: RequestedTestFilter(EAutomationTestFlags::SmokeFilter)
	, StartTime(0.0f)
	, bTestSuccessful(false)
	, CurrentTest(nullptr)
	, bDeveloperDirectoryIncluded(false)
	, NetworkRoleIndex(0)
	, bForceSmokeTests(false)
	, bCaptureStack(true)
{
	TagFilter = MakeShared<FTextFilterExpressionEvaluator>(ETextFilterExpressionEvaluatorMode::BasicString);
}

FAutomationTestFramework::~FAutomationTestFramework()
{
	AutomationTestClassNameToInstanceMap.Empty();
}

FString FAutomationExecutionEntry::ToString() const
{
	FString ComplexString;

	ComplexString = Event.Message;

	if (!Event.Context.IsEmpty())
	{
		ComplexString += TEXT(" [");
		ComplexString += Event.Context;
		ComplexString += TEXT("] ");
	}

	// Place the filename at the end so it can be extracted by the SAutomationWindow widget
	// Expectation is "[filename(line)]"
	const FString FileLineDescription = AutomationTest::CreateFileLineDescription(Filename, LineNumber);
	if ( !FileLineDescription.IsEmpty() )
	{
		ComplexString += TEXT(" [");
		ComplexString += FileLineDescription;
		ComplexString += TEXT("]");
	}

	return ComplexString;
}

FString FAutomationExecutionEntry::ToStringFormattedEditorLog() const
{
	FString ComplexString;

	ComplexString = Event.Message;

	if (!Event.Context.IsEmpty())
	{
		ComplexString += TEXT(" [");
		ComplexString += Event.Context;
		ComplexString += TEXT("] ");
	}

	const FString FileLineDescription = AutomationTest::CreateFileLineDescription(Filename, LineNumber);
	if (!FileLineDescription.IsEmpty())
	{
		ComplexString += TEXT(" ");
		ComplexString += FileLineDescription;
	}

	return ComplexString;
}
//------------------------------------------------------------------------------

void FAutomationTestExecutionInfo::Clear()
{
	ContextStack.Reset();

	Entries.Empty();
	AnalyticsItems.Empty();
	TelemetryItems.Empty();
	TelemetryStorage.Empty();

	Errors = 0;
	Warnings = 0;
}

int32 FAutomationTestExecutionInfo::RemoveAllEvents(EAutomationEventType EventType)
{
	return RemoveAllEvents([EventType] (const FAutomationEvent& Event) {
		return Event.Type == EventType;
	});
}

int32 FAutomationTestExecutionInfo::RemoveAllEvents(TFunctionRef<bool(FAutomationEvent&)> FilterPredicate)
{
	int32 TotalRemoved = Entries.RemoveAll([this, &FilterPredicate](FAutomationExecutionEntry& Entry) {
		if (FilterPredicate(Entry.Event))
		{
			switch (Entry.Event.Type)
			{
			case EAutomationEventType::Warning:
				Warnings--;
				break;
			case EAutomationEventType::Error:
				Errors--;
				break;
			}

			return true;
		}
		return false;
	});

	return TotalRemoved;
}

void FAutomationTestExecutionInfo::AddEvent(const FAutomationEvent& Event, int StackOffset, bool bCaptureStack)
{
	LLM_SCOPE_BYNAME(TEXT("AutomationTest/Framework"));

	switch (Event.Type)
	{
	case EAutomationEventType::Warning:
		Warnings++;
		break;
	case EAutomationEventType::Error:
		Errors++;
		break;
	}

	int32 EntryIndex = -1;
	if (FAutomationTestFramework::Get().GetCaptureStack() && bCaptureStack)
	{
		SAFE_GETSTACK(Stack, StackOffset + 1, 1);
		if (Stack.Num())
		{
			EntryIndex = Entries.Add(FAutomationExecutionEntry(Event, Stack[0].Filename, Stack[0].LineNumber));
		}
	}
	if (EntryIndex == -1)
	{
		EntryIndex = Entries.Add(FAutomationExecutionEntry(Event));
	}

	FAutomationExecutionEntry& NewEntry = Entries[EntryIndex];

	if (NewEntry.Event.Context.IsEmpty())
	{
		NewEntry.Event.Context = GetContext();
	}
}

void FAutomationTestExecutionInfo::AddWarning(const FString& WarningMessage)
{
	AddEvent(FAutomationEvent(EAutomationEventType::Warning, WarningMessage));
}

void FAutomationTestExecutionInfo::AddError(const FString& ErrorMessage)
{
	AddEvent(FAutomationEvent(EAutomationEventType::Error, ErrorMessage));
}

//------------------------------------------------------------------------------

FAutomationEvent FAutomationScreenshotCompareResults::ToAutomationEvent() const
{
	FAutomationEvent Event(EAutomationEventType::Info, TEXT(""));

	if (bWasNew)
	{
		Event.Type = EAutomationEventType::Warning;
		Event.Message = FString::Printf(TEXT("New Screenshot '%s' was discovered!  Please add a ground truth version of it."), *ScreenshotPath);
	}
	else
	{
		if (bWasSimilar)
		{
			Event.Type = EAutomationEventType::Info;
			Event.Message = FString::Printf(TEXT("Screenshot '%s' was similar!  Global Difference = %f, Max Local Difference = %f"),
				*ScreenshotPath, GlobalDifference, MaxLocalDifference);
		}
		else
		{
			Event.Type = EAutomationEventType::Error;

			if (ErrorMessage.IsEmpty())
			{
				Event.Message = FString::Printf(TEXT("Screenshot '%s' test failed, Screenshots were different!  Global Difference = %f, Max Local Difference = %f"),
					*ScreenshotPath, GlobalDifference, MaxLocalDifference);
			}
			else
			{
				Event.Message = FString::Printf(TEXT("Screenshot '%s' test failed; Error = %s"), *ScreenshotPath, *ErrorMessage);
			}
		}
	}

	Event.Artifact = UniqueId;
	return Event;
}

//------------------------------------------------------------------------------

void FAutomationTestBase::ClearExecutionInfo()
{
	ExecutionInfo.Clear();
}

UE_AUTORTFM_ALWAYS_OPEN
void FAutomationTestBase::AddError(const FString& InError, int32 StackOffset)
{
	if( !IsExpectedMessage(InError, ELogVerbosity::Warning))
	{
		UE::TWriteScopeLock Lock(ActionCS);
		ExecutionInfo.AddEvent(FAutomationEvent(EAutomationEventType::Error, InError), StackOffset + 1);
	}
}

UE_AUTORTFM_ALWAYS_OPEN
bool FAutomationTestBase::AddErrorIfFalse(bool bCondition, const FString& InError, int32 StackOffset)
{
	if (!bCondition)
	{
		AddError(InError, StackOffset + 1);
	}
	return bCondition;
}

UE_AUTORTFM_ALWAYS_OPEN
void FAutomationTestBase::AddWarning( const FString& InWarning, int32 StackOffset )
{
	if ( !IsExpectedMessage(InWarning, ELogVerbosity::Warning))
	{
		UE::TWriteScopeLock Lock(ActionCS);
		ExecutionInfo.AddEvent(FAutomationEvent(EAutomationEventType::Warning, InWarning), StackOffset + 1);
	}
}

UE_AUTORTFM_ALWAYS_OPEN
void FAutomationTestBase::AddInfo( const FString& InLogItem, int32 StackOffset, bool bCaptureStack )
{
	if ( !IsExpectedMessage(InLogItem, ELogVerbosity::Display))
	{
		UE::TWriteScopeLock Lock(ActionCS);
		ExecutionInfo.AddEvent(FAutomationEvent(EAutomationEventType::Info, InLogItem), StackOffset + 1, bCaptureStack);
	}
}

UE_AUTORTFM_ALWAYS_OPEN
void FAutomationTestBase::AddAnalyticsItem(const FString& InAnalyticsItem)
{
	UE::TWriteScopeLock Lock(ActionCS);
	ExecutionInfo.AnalyticsItems.Add(InAnalyticsItem);
}

UE_AUTORTFM_ALWAYS_OPEN
void FAutomationTestBase::AddTelemetryData(const FString& DataPoint, double Measurement, const FString& Context)
{
	UE::TWriteScopeLock Lock(ActionCS);
	ExecutionInfo.TelemetryItems.Add(FAutomationTelemetryData(DataPoint, Measurement, Context));
}

UE_AUTORTFM_ALWAYS_OPEN
void FAutomationTestBase::AddTelemetryData(const TMap <FString, double>& ValuePairs, const FString& Context)
{
	UE::TWriteScopeLock Lock(ActionCS);
	for (const TPair<FString, double>& Item : ValuePairs)
	{
		ExecutionInfo.TelemetryItems.Add(FAutomationTelemetryData(Item.Key, Item.Value, Context));
	}
}

void FAutomationTestBase::SetTelemetryStorage(const FString& StorageName)
{
	ExecutionInfo.TelemetryStorage = StorageName;
}

UE_AUTORTFM_ALWAYS_OPEN
void FAutomationTestBase::AddEvent(const FAutomationEvent& InEvent, int32 StackOffset, bool bCaptureStack)
{
	ELogVerbosity::Type LogType = ELogVerbosity::Display;
	if (InEvent.Type == EAutomationEventType::Error)
	{
		LogType = ELogVerbosity::Error;
	}
	else if (InEvent.Type == EAutomationEventType::Warning)
	{
		LogType = ELogVerbosity::Warning;
	}

	if (!IsExpectedMessage(InEvent.Message, LogType))
	{
		UE::TWriteScopeLock Lock(ActionCS);
		ExecutionInfo.AddEvent(InEvent, StackOffset + 1, bCaptureStack);
	}
}

bool FAutomationTestBase::HasAnyErrors() const
{
	return ExecutionInfo.GetErrorTotal() > 0;
}

bool FAutomationTestBase::HasMetExpectedMessages(ELogVerbosity::Type VerbosityType)
{
	bool bHasMetAllExpectedMessages = true;
	TArray<FAutomationExpectedMessage> ExpectedMessagesArray;
	{
		UE::TReadScopeLock RLock(ActionCS);
		ExpectedMessagesArray = ExpectedMessages.Array();
	}
	for (FAutomationExpectedMessage& ExpectedMessage : ExpectedMessagesArray)
	{
		if (!LogCategoryMatchesSeverityInclusive(ExpectedMessage.Verbosity, VerbosityType))
		{
			continue;
		}

		// Avoid ambiguity of the messages below when the verbosity is "All"
		const TCHAR* LogVerbosityString = ExpectedMessage.Verbosity == ELogVerbosity::All ? TEXT("Any") : ToString(ExpectedMessage.Verbosity);

		if (ExpectedMessage.ExpectedNumberOfOccurrences > 0 && (ExpectedMessage.ExpectedNumberOfOccurrences != ExpectedMessage.ActualNumberOfOccurrences))
		{
			UE::TWriteScopeLock WLock(ActionCS);
			bHasMetAllExpectedMessages = false;

			ExecutionInfo.AddEvent(FAutomationEvent(EAutomationEventType::Error,
				FString::Printf(TEXT("Expected ('%s') level log message or higher matching '%s' to occur %d times with %s match type, but it was found %d time(s).")
					, LogVerbosityString
					, *ExpectedMessage.MessagePatternString
					, ExpectedMessage.ExpectedNumberOfOccurrences
					, EAutomationExpectedMessageFlags::ToString(ExpectedMessage.CompareType)
					, ExpectedMessage.ActualNumberOfOccurrences)
				, ExecutionInfo.GetContext()));
		}
		else if (ExpectedMessage.ExpectedNumberOfOccurrences == 0)
		{
			UE::TWriteScopeLock WLock(ActionCS);
			if (ExpectedMessage.ActualNumberOfOccurrences == 0)
			{
				bHasMetAllExpectedMessages = false;

				ExecutionInfo.AddEvent(FAutomationEvent(EAutomationEventType::Error,
					FString::Printf(TEXT("Expected suppressed ('%s') level log message or higher matching '%s' did not occur.")
						, LogVerbosityString
						, *ExpectedMessage.MessagePatternString)
					, ExecutionInfo.GetContext()));
			}
			else
			{
				ExecutionInfo.AddEvent(FAutomationEvent(EAutomationEventType::Info,
					FString::Printf(TEXT("Suppressed expected ('%s') level log message or higher matching '%s' %d times.")
						, LogVerbosityString
						, *ExpectedMessage.MessagePatternString
						, ExpectedMessage.ActualNumberOfOccurrences)
					, ExecutionInfo.GetContext()));
			}
		}
	}

	return bHasMetAllExpectedMessages;
}

bool FAutomationTestBase::HasMetExpectedErrors()
{
	return HasMetExpectedMessages(ELogVerbosity::Warning);
}

void FAutomationTestBase::InternalSetSuccessState( bool bSuccessful )
{
	ExecutionInfo.bSuccessful = bSuccessful;
}

FString FAutomationTestBase::GetStringValueToDisplay(FStringView Value) const
{
	if (Value.GetData())
	{
		return FString::Printf(TEXT("\"%.*s\""), Value.Len(), Value.GetData());
	}
	else
	{
		return TEXT("nullptr");
	}
}

FString FAutomationTestBase::GetStringValueToDisplay(FUtf8StringView Value) const
{
	if (Value.GetData())
	{
		return FString::Printf(TEXT("\"%s\""), *WriteToString<128>(Value));
	}
	else
	{
		return TEXT("nullptr");
	}
}

bool FAutomationTestBase::GetLastExecutionSuccessState()
{
	return ExecutionInfo.bSuccessful;
}

void FAutomationTestBase::GetExecutionInfo( FAutomationTestExecutionInfo& OutInfo ) const
{
	OutInfo = ExecutionInfo;
}

void FAutomationTestBase::AddExpectedMessage(
	FString ExpectedPatternString,
	ELogVerbosity::Type ExpectedVerbosity,
	EAutomationExpectedMessageFlags::MatchType CompareType,
	int32 Occurrences,
	bool IsRegex)
{
	UE::TWriteScopeLock Lock(ActionCS);
	ExpectedMessages.Add(FAutomationExpectedMessage(ExpectedPatternString, ExpectedVerbosity, CompareType, Occurrences, IsRegex));
}

void FAutomationTestBase::AddExpectedMessage(
	FString ExpectedPatternString,
	EAutomationExpectedMessageFlags::MatchType CompareType,
	int32 Occurrences,
	bool IsRegex)
{
	AddExpectedMessage(MoveTemp(ExpectedPatternString), ELogVerbosity::All, CompareType, Occurrences, IsRegex);	
}

void FAutomationTestBase::AddExpectedMessagePlain(
	FString ExpectedString,
	ELogVerbosity::Type ExpectedVerbosity,
	EAutomationExpectedMessageFlags::MatchType CompareType,
	int32 Occurrences)
{
	AddExpectedMessage(MoveTemp(ExpectedString), ExpectedVerbosity, CompareType, Occurrences, false);
}

void FAutomationTestBase::AddExpectedMessagePlain(
	FString ExpectedString,
	EAutomationExpectedMessageFlags::MatchType CompareType,
	int32 Occurrences)
{
	AddExpectedMessagePlain(MoveTemp(ExpectedString), ELogVerbosity::All, CompareType, Occurrences);
}

void FAutomationTestBase::GetExpectedMessages(
	TArray<FAutomationExpectedMessage>& OutInfo,
	ELogVerbosity::Type Verbosity) const
{
	// Do not include any suppressed messages
	GetExpectedMessages(OutInfo, false, Verbosity);
}

void FAutomationTestBase::GetExpectedMessages(
	TArray<FAutomationExpectedMessage>& OutInfo,
	bool IncludeSuppressed,
	ELogVerbosity::Type Verbosity) const
{
	OutInfo.Reserve(ExpectedMessages.Num());
	Algo::CopyIf(ExpectedMessages, OutInfo, [IncludeSuppressed, Verbosity](const FAutomationExpectedMessage& Message)
		{
			const bool bIsIncluded = (Message.ExpectedNumberOfOccurrences < 0) ? IncludeSuppressed : true;
			return bIsIncluded && FAutomationTestBase::LogCategoryMatchesSeverityInclusive(Message.Verbosity, Verbosity);
		});
	OutInfo.Sort();
}

void FAutomationTestBase::AddExpectedError(FString ExpectedErrorPattern, EAutomationExpectedErrorFlags::MatchType InCompareType, int32 Occurrences, bool IsRegex)
{
	// Set verbosity to Warning as it's inclusive, and so checks for both Warnings and Errors
	AddExpectedMessage(MoveTemp(ExpectedErrorPattern), ELogVerbosity::Warning, static_cast<EAutomationExpectedMessageFlags::MatchType>(InCompareType), Occurrences, IsRegex);
}

void FAutomationTestBase::AddExpectedErrorPlain(
	FString ExpectedString,
	EAutomationExpectedErrorFlags::MatchType CompareType,
	int32 Occurrences)
{
	AddExpectedMessagePlain(MoveTemp(ExpectedString), ELogVerbosity::Warning, static_cast<EAutomationExpectedMessageFlags::MatchType>(CompareType), Occurrences);
}

EAutomationTestFlags FAutomationTestBase::ExtractAutomationTestFlags(FString InTagNotation)
{
	EAutomationTestFlags Result = EAutomationTestFlags::None;
	TArray<FString> OutputParts;
	InTagNotation
		.Replace(TEXT("["), TEXT(""))
		.Replace(TEXT("]"), TEXT(";"))
		.ParseIntoArray(OutputParts, TEXT(";"), true);
	for (const FString& Part : OutputParts)
	{
		Result |= EAutomationTestFlags_GetTestFlagsMap().FindRef(Part, EAutomationTestFlags::None);
	}
	return Result;
}

void FAutomationTestBase::GenerateTestNames(TArray<FAutomationTestInfo>& TestInfo) const
{
	// This can take a while, particularly as spec tests walk the callstack, so suspend the heartbeat watchdog and hitch detector
	FSlowHeartBeatScope SuspendHeartBeat;
	FDisableHitchDetectorScope SuspendGameThreadHitch;

	TArray<FString> BeautifiedNames;
	TArray<FString> ParameterNames;
	GetTests(BeautifiedNames, ParameterNames);
	FAutomationTestFramework& Framework = FAutomationTestFramework::Get();

	FString BeautifiedTestName = GetBeautifiedTestName();

	for (int32 ParameterIndex = 0; ParameterIndex < ParameterNames.Num(); ++ParameterIndex)
	{
		FString CompleteBeautifiedNames = BeautifiedTestName;
		FString CompleteTestName = TestName;

		if (ParameterNames[ParameterIndex].Len())
		{
			CompleteBeautifiedNames = FString::Printf(TEXT("%s.%s"), *BeautifiedTestName, *BeautifiedNames[ParameterIndex]);
			CompleteTestName = FString::Printf(TEXT("%s %s"), *TestName, *ParameterNames[ParameterIndex]);
		}

		// Add the test info to our collection
		FAutomationTestInfo NewTestInfo(
			CompleteBeautifiedNames,
			CompleteBeautifiedNames,
			CompleteTestName,
			GetTestFlags(),
			GetRequiredDeviceNum(),
			ParameterNames[ParameterIndex],
			GetTestSourceFileName(CompleteTestName),
			GetTestSourceFileLine(CompleteTestName),
			GetTestAssetPath(ParameterNames[ParameterIndex]),
			GetTestOpenCommand(ParameterNames[ParameterIndex]),
			Framework.GetTagsForAutomationTest(CompleteBeautifiedNames)
		);
		
		TestInfo.Add( NewTestInfo );
	}
}

bool FAutomationTestBase::LogCategoryMatchesSeverityInclusive(
	ELogVerbosity::Type Actual,
	ELogVerbosity::Type MaximumVerbosity)
{
	// Special case for "all", which should always match
	return Actual == ELogVerbosity::All || MaximumVerbosity == ELogVerbosity::All || Actual <= MaximumVerbosity;
}

void FAutomationTestBase::LoadDefaultLogSettings()
{
	GConfig->GetBool(TEXT("/Script/AutomationController.AutomationControllerSettings"), TEXT("bSuppressLogErrors"), bSuppressLogErrors, GEngineIni);
	GConfig->GetBool(TEXT("/Script/AutomationController.AutomationControllerSettings"), TEXT("bSuppressLogWarnings"), bSuppressLogWarnings, GEngineIni);
	GConfig->GetBool(TEXT("/Script/AutomationController.AutomationControllerSettings"), TEXT("bElevateLogWarningsToErrors"), bElevateLogWarningsToErrors, GEngineIni);
	GConfig->GetArray(TEXT("/Script/AutomationController.AutomationControllerSettings"), TEXT("SuppressedLogCategories"), SuppressedLogCategories, GEngineIni);
}

// --------------------------------------------------------------------------------------

bool FAutomationTestBase::TestEqual(const TCHAR* What, const int32 Actual, const int32 Expected)
{
	if (Actual != Expected)
	{
		AddError(FString::Printf(TEXT("Expected '%s' to be %d, but it was %d."), What, Expected, Actual), 1);
		return false;
	}
	return true;
}

bool FAutomationTestBase::TestEqual(const TCHAR* What, const int64 Actual, const int64 Expected)
{
	if (Actual != Expected)
	{
		AddError(FString::Printf(TEXT("Expected '%s' to be %" PRId64 ", but it was %" PRId64 "."), What, Expected, Actual), 1);
		return false;
	}
	return true;
}

bool FAutomationTestBase::TestEqual(const TCHAR* What, const SIZE_T Actual, const SIZE_T Expected)
{
	if (Actual != Expected)
	{
		AddError(FString::Printf(TEXT("Expected '%s' to be %" PRIuPTR ", but it was %" PRIuPTR "."), What, Expected, Actual), 1);
		return false;
	}
	return true;
}

bool FAutomationTestBase::TestEqual(const TCHAR* What, const float Actual, const float Expected, float Tolerance)
{
	if (!FMath::IsNearlyEqual(Actual, Expected, Tolerance))
	{
		AddError(FString::Printf(TEXT("Expected '%s' to be %f, but it was %f and outside tolerance %f."), What, Expected, Actual, Tolerance), 1);
		return false;
	}
	return true;
}

bool FAutomationTestBase::TestEqual(const TCHAR* What, const double Actual, const double Expected, double Tolerance)
{
	if (!FMath::IsNearlyEqual(Actual, Expected, Tolerance))
	{
		AddError(FString::Printf(TEXT("Expected '%s' to be %f, but it was %f and outside tolerance %f."), What, Expected, Actual, Tolerance), 1);
		return false;
	}
	return true;
}

bool FAutomationTestBase::TestEqual(const TCHAR* What, const FVector Actual, const FVector Expected, float Tolerance)
{
	if (!Expected.Equals(Actual, Tolerance))
	{
		AddError(FString::Printf(TEXT("Expected '%s' to be %s, but it was %s and outside tolerance %f."), What, *Expected.ToString(), *Actual.ToString(), Tolerance), 1);
		return false;
	}
	return true;
}

bool FAutomationTestBase::TestEqual(const TCHAR* What, const FTransform Actual, const FTransform Expected, float Tolerance)
{
	if (!Expected.Equals(Actual, Tolerance))
	{
		AddError(FString::Printf(TEXT("Expected '%s' to be %s, but it was %s and outside tolerance %f."), What, *Expected.ToString(), *Actual.ToString(), Tolerance), 1);
		return false;
	}
	return true;
}

bool FAutomationTestBase::TestEqual(const TCHAR* What, const FRotator Actual, const FRotator Expected, float Tolerance)
{
	if (!Expected.Equals(Actual, Tolerance))
	{
		AddError(FString::Printf(TEXT("Expected '%s' to be %s, but it was %s and outside tolerance %f."), What, *Expected.ToString(), *Actual.ToString(), Tolerance), 1);
		return false;
	}
	return true;
}

bool FAutomationTestBase::TestEqual(const TCHAR* What, const FColor Actual, const FColor Expected)
{
	if (Expected != Actual)
	{
		AddError(FString::Printf(TEXT("Expected '%s' to be %s, but it was %s."), What, *Expected.ToString(), *Actual.ToString()), 1);
		return false;
	}
	return true;
}

bool FAutomationTestBase::TestEqual(const TCHAR* What, const FLinearColor Actual, const FLinearColor Expected)
{
	if (Expected != Actual)
	{
		AddError(FString::Printf(TEXT("Expected '%s' to be %s, but it was %s."), What, *Expected.ToString(), *Actual.ToString()), 1);
		return false;
	}
	return true;
}

bool FAutomationTestBase::TestEqual(const TCHAR* What, const TCHAR* Actual, const TCHAR* Expected)
{
 	bool bAreEqual = (Actual && Expected) ? (FCString::Stricmp(Actual, Expected) == 0) : (Actual == Expected);
 
 	if (!bAreEqual)
 	{
 		AddError(FString::Printf(TEXT("Expected '%s' to be %s, but it was %s."), What, *GetStringValueToDisplay(Expected), *GetStringValueToDisplay(Actual)), 1);
 	}
 
 	return bAreEqual;
}

bool FAutomationTestBase::TestEqual(const TCHAR* What, FUtf8StringView Actual, FUtf8StringView Expected)
{
	if (Actual.Compare(Expected, ESearchCase::IgnoreCase) != 0)
	{
		AddError(FString::Printf(TEXT("Expected '%s' to be %s, but it was %s."), What, *GetStringValueToDisplay(Expected), *GetStringValueToDisplay(Actual)), 1);
		return false;
	}

	return true;
}

bool FAutomationTestBase::TestEqual(const TCHAR* What, FStringView Actual, FStringView Expected)
{
	if (Actual.Compare(Expected, ESearchCase::IgnoreCase) != 0)
	{
		AddError(FString::Printf(TEXT("Expected '%s' to be %s, but it was %s."), What, *GetStringValueToDisplay(Expected), *GetStringValueToDisplay(Actual)), 1);
		return false;
	}

	return true;
}

bool FAutomationTestBase::TestEqual(const TCHAR* What, const FText Actual, const FText Expected)
{
	if (!Actual.EqualToCaseIgnored(Expected))
	{
		AddError(FString::Printf(TEXT("Expected '%s' to be %s, but it was %s."), What, *Expected.ToString(), *Actual.ToString()), 1);
		return false;
	}

	return true;
}

bool FAutomationTestBase::TestEqual(const TCHAR* What, const FName Actual, const FName Expected)
{
	if (!Actual.IsEqual(Expected))
	{
		AddError(FString::Printf(TEXT("Expected '%s' to be %s, but it was %s."), What, *Expected.ToString(), *Actual.ToString()), 1);
		return false;
	}

	return true;
}

bool FAutomationTestBase::TestNotEqual(const TCHAR* What, const TCHAR* Actual, const TCHAR* Expected)
{
	bool bAreDifferent = (Actual && Expected) ? (FCString::Stricmp(Actual, Expected) != 0) : (Actual != Expected);

	if (!bAreDifferent)
	{
		AddError(FString::Printf(TEXT("Expected '%s' to differ from %s, but it was %s."), What, *GetStringValueToDisplay(Expected), *GetStringValueToDisplay(Actual)), 1);
	}

 	return bAreDifferent;
}

bool FAutomationTestBase::TestNotEqual(const TCHAR* What, FUtf8StringView Actual, FUtf8StringView Expected)
{
	if (Actual.Compare(Expected, ESearchCase::IgnoreCase) == 0)
	{
		AddError(FString::Printf(TEXT("Expected '%s' to differ from %s, but it was %s."), What, *GetStringValueToDisplay(Expected), *GetStringValueToDisplay(Actual)), 1);
		return false;
	}

	return true;
}

bool FAutomationTestBase::TestNotEqual(const TCHAR* What, FStringView Actual, FStringView Expected)
{
	if (Actual.Compare(Expected, ESearchCase::IgnoreCase) == 0)
	{
		AddError(FString::Printf(TEXT("Expected '%s' to differ from %s, but it was %s."), What, *GetStringValueToDisplay(Expected), *GetStringValueToDisplay(Actual)), 1);
		return false;
	}

	return true;
}

bool FAutomationTestBase::TestNotEqual(const TCHAR* What, const float Actual, const float Expected, float Tolerance)
{
	if (FMath::IsNearlyEqual(Actual, Expected, Tolerance))
	{
		AddError(FString::Printf(TEXT("Expected '%s' to be unequal to %f, but it was %f and within tolerance %f."), What, Expected, Actual, Tolerance), 1);
		return false;
	}
	return true;
}

bool FAutomationTestBase::TestNotEqual(const TCHAR* What, const double Actual, const double Expected, double Tolerance)
{
	if (FMath::IsNearlyEqual(Actual, Expected, Tolerance))
	{
		AddError(FString::Printf(TEXT("Expected '%s' to be unequal to %f, but it was %f and within tolerance %f."), What, Expected, Actual, Tolerance), 1);
		return false;
	}
	return true;
}

bool FAutomationTestBase::TestNotEqual(const TCHAR* What, const FText Actual, const FText Expected)
{
	if (Actual.EqualToCaseIgnored(Expected))
	{
		AddError(FString::Printf(TEXT("Expected '%s' to differ from %s, but it was %s."), What, *Expected.ToString(), *Actual.ToString()), 1);
		return false;
	}

	return true;
}

bool FAutomationTestBase::TestNotEqual(const TCHAR* What, const FName Actual, const FName Expected)
{
	if (Actual.IsEqual(Expected))
	{
		AddError(FString::Printf(TEXT("Expected '%s' to differ from %s, but it was %s."), What, *Expected.ToString(), *Actual.ToString()), 1);
		return false;
	}

	return true;
}

bool FAutomationTestBase::TestEqualSensitive(const TCHAR* What, const TCHAR* Actual, const TCHAR* Expected)
{
 	bool bAreEqual = (Actual && Expected) ? (FCString::Strcmp(Actual, Expected) == 0) : (Actual == Expected);
 
 	if (!bAreEqual)
 	{
 		AddError(FString::Printf(TEXT("Expected '%s' to be %s, but it was %s."), What, *GetStringValueToDisplay(Expected), *GetStringValueToDisplay(Actual)), 1);
 	}
 
 	return bAreEqual;
}

bool FAutomationTestBase::TestEqualSensitive(const TCHAR* What, FStringView Actual, FStringView Expected)
{
	if (Actual.Compare(Expected, ESearchCase::CaseSensitive) != 0)
	{
		AddError(FString::Printf(TEXT("Expected '%s' to be %s, but it was %s."), What, *GetStringValueToDisplay(Expected), *GetStringValueToDisplay(Actual)), 1);
		return false;
	}

	return true;
}

bool FAutomationTestBase::TestEqualSensitive(const TCHAR* What, FUtf8StringView Actual, FUtf8StringView Expected)
{
	if (Actual.Compare(Expected, ESearchCase::CaseSensitive) != 0)
	{
		AddError(FString::Printf(TEXT("Expected '%s' to be %s, but it was %s."), What, *GetStringValueToDisplay(Expected), *GetStringValueToDisplay(Actual)), 1);
		return false;
	}

	return true;
}

bool FAutomationTestBase::TestNotEqualSensitive(const TCHAR* What, const TCHAR* Actual, const TCHAR* Expected)
{
 	bool bAreDifferent = (Actual && Expected) ? (FCString::Strcmp(Actual, Expected) != 0) : (Actual != Expected);
 
 	if (!bAreDifferent)
 	{
 		AddError(FString::Printf(TEXT("Expected '%s' to differ from %s, but it was %s."), What, *GetStringValueToDisplay(Expected), *GetStringValueToDisplay(Actual)), 1);
 	}
 
 	return bAreDifferent;
}

bool FAutomationTestBase::TestNotEqualSensitive(const TCHAR* What, FStringView Actual, FStringView Expected)
{
	if (Actual.Compare(Expected, ESearchCase::CaseSensitive) == 0)
	{
		AddError(FString::Printf(TEXT("Expected '%s' to differ from %s, but it was %s."), What, *GetStringValueToDisplay(Expected), *GetStringValueToDisplay(Actual)), 1);
		return false;
	}

	return true;
}

bool FAutomationTestBase::TestNotEqualSensitive(const TCHAR* What, FUtf8StringView Actual, FUtf8StringView Expected)
{
	if (Actual.Compare(Expected, ESearchCase::CaseSensitive) == 0)
	{
		AddError(FString::Printf(TEXT("Expected '%s' to differ from %s, but it was %s."), What, *GetStringValueToDisplay(Expected), *GetStringValueToDisplay(Actual)), 1);
		return false;
	}

	return true;
}

bool FAutomationTestBase::TestNearlyEqual(const TCHAR* What, const float Actual, const float Expected, float Tolerance)
{
	return TestEqual(What, Actual, Expected, Tolerance);
}

bool FAutomationTestBase::TestNearlyEqual(const TCHAR* What, const double Actual, const double Expected, double Tolerance)
{
	return TestEqual(What, Actual, Expected, Tolerance);
}

bool FAutomationTestBase::TestNearlyEqual(const TCHAR* What, const FVector Actual, const FVector Expected, float Tolerance)
{
	return TestEqual(What, Actual, Expected, Tolerance);
}

bool FAutomationTestBase::TestNearlyEqual(const TCHAR* What, const FTransform Actual, const FTransform Expected, float Tolerance)
{
	return TestEqual(What, Actual, Expected, Tolerance);
}

bool FAutomationTestBase::TestNearlyEqual(const TCHAR* What, const FRotator Actual, const FRotator Expected, float Tolerance)
{
	return TestEqual(What, Actual, Expected, Tolerance);
}

bool FAutomationTestBase::TestLessThan(const TCHAR* What, const int32 Actual, const int32 Expected)
{
	if (Actual < Expected)
	{
		return true;
	}
	AddError(FString::Printf(TEXT("Expected '%s' to be less than %d, but it was %d."), What, Expected, Actual), 1);
	return false;
}

bool FAutomationTestBase::TestLessThan(const TCHAR* What, const int64 Actual, const int64 Expected)
{
	if (Actual < Expected)
	{
		return true;
	}	
	AddError(FString::Printf(TEXT("Expected '%s' to be less than %" PRId64 ", but it was %" PRId64 "."), What, Expected, Actual), 1);
	return false;
}

bool FAutomationTestBase::TestGreaterThan(const TCHAR* What, const int32 Actual, const int32 Expected)
{
	if (Actual > Expected)
	{
		return true;
	}
	AddError(FString::Printf(TEXT("Expected '%s' to be greater than %d, but it was %d."), What, Expected, Actual), 1);
	return false;
}

bool FAutomationTestBase::TestGreaterThan(const TCHAR* What, const int64 Actual, const int64 Expected)
{
	if (Actual > Expected)
	{
		return true;
	}
	AddError(FString::Printf(TEXT("Expected '%s' to be greater than %" PRId64 ", but it was %" PRId64 "."), What, Expected, Actual), 1);
	return false;
}

bool FAutomationTestBase::TestLessEqual(const TCHAR* What, const int32 Actual, const int32 Expected)
{
	if (Actual <= Expected)
	{
		return true;
	}
	AddError(FString::Printf(TEXT("Expected '%s' to be less than or equal to %d, but it was %d."), What, Expected, Actual), 1);
	return false;
}

bool FAutomationTestBase::TestLessEqual(const TCHAR* What, const int64 Actual, const int64 Expected)
{
	if (Actual <= Expected)
	{
		return true;
	}
	AddError(FString::Printf(TEXT("Expected '%s' to be less than or equal to %" PRId64 ", but it was %" PRId64 "."), What, Expected, Actual), 1);
	return false;
}

bool FAutomationTestBase::TestGreaterEqual(const TCHAR* What, const int32 Actual, const int32 Expected)
{
	if (Actual >= Expected)
	{
		return true;
	}
	AddError(FString::Printf(TEXT("Expected '%s' to be greater than or equal to %d, but it was %d."), What, Expected, Actual), 1);
	return false;
}

bool FAutomationTestBase::TestGreaterEqual(const TCHAR* What, const int64 Actual, const int64 Expected)
{
	if (Actual >= Expected)
	{
		return true;
	}
	AddError(FString::Printf(TEXT("Expected '%s' to be greater than or equal to %" PRId64 ", but it was %" PRId64 "."), What, Expected, Actual), 1);
	return false;
}

bool FAutomationTestBase::TestLessThan(const TCHAR* What, const SIZE_T Actual, const SIZE_T Expected)
{
	if (Actual < Expected)
	{
		return true;
	}
	AddError(FString::Printf(TEXT("Expected '%s' to be less than %" PRIuPTR ", but it was %" PRIuPTR "."), What, Expected, Actual), 1);
	return false;
}

bool FAutomationTestBase::TestGreaterThan(const TCHAR* What, const SIZE_T Actual, const SIZE_T Expected)
{
	if (Actual > Expected)
	{
		return true;
	}
	AddError(FString::Printf(TEXT("Expected '%s' to be greater than %" PRIuPTR ", but it was %" PRIuPTR "."), What, Expected, Actual), 1);
	return false;
}

bool FAutomationTestBase::TestLessEqual(const TCHAR* What, const SIZE_T Actual, const SIZE_T Expected)
{
	if (Actual <= Expected)
	{
		return true;
	}
	AddError(FString::Printf(TEXT("Expected '%s' to be less than or equal to %" PRIuPTR ", but it was %" PRIuPTR "."), What, Expected, Actual), 1);
	return false;
}

bool FAutomationTestBase::TestGreaterEqual(const TCHAR* What, const SIZE_T Actual, const SIZE_T Expected)
{
	if (Actual >= Expected)
	{
		return true;
	}
	AddError(FString::Printf(TEXT("Expected '%s' to be greater than or equal to %" PRIuPTR ", but it was %" PRIuPTR "."), What, Expected, Actual), 1);
	return false;
}

bool FAutomationTestBase::TestLessThan(const TCHAR* What, const float Actual, const float Expected, float Tolerance)
{
	if (FMath::IsNearlyEqual(Actual, Expected, Tolerance))
	{
		AddError(FString::Printf(TEXT("Expected '%s' to be less than %f, but it was %f and within equality tolerance %f."), What, Expected, Actual, Tolerance), 1);
		return false;
	}
	if (Actual < Expected)
	{
		return true;
	}
	AddError(FString::Printf(TEXT("Expected '%s' to be less than %f, but it was %f and outside equality tolerance %f."), What, Expected, Actual, Tolerance), 1);
	return false;
}

bool FAutomationTestBase::TestLessThan(const TCHAR* What, const double Actual, const double Expected, double Tolerance)
{
	if (FMath::IsNearlyEqual(Actual, Expected, Tolerance))
	{
		AddError(FString::Printf(TEXT("Expected '%s' to be less than %f, but it was %f and within equality tolerance %f."), What, Expected, Actual, Tolerance), 1);
		return false;
	}
	if (Actual < Expected)
	{
		return true;
	}
	AddError(FString::Printf(TEXT("Expected '%s' to be less than %f, but it was %f and outside equality tolerance %f."), What, Expected, Actual, Tolerance), 1);
	return false;
}

bool FAutomationTestBase::TestGreaterThan(const TCHAR* What, const float Actual, const float Expected, float Tolerance)
{
	if (FMath::IsNearlyEqual(Actual, Expected, Tolerance))
	{
		AddError(FString::Printf(TEXT("Expected '%s' to be greater than %f, but it was %f and within equality tolerance %f."), What, Expected, Actual, Tolerance), 1);
		return false;
	}
	if (Actual > Expected)
	{
		return true;
	}
	AddError(FString::Printf(TEXT("Expected '%s' to be greater than %f, but it was %f and outside equality tolerance %f."), What, Expected, Actual, Tolerance), 1);
	return false;
}

bool FAutomationTestBase::TestGreaterThan(const TCHAR* What, const double Actual, const double Expected, double Tolerance)
{
	if (FMath::IsNearlyEqual(Actual, Expected, Tolerance))
	{
		AddError(FString::Printf(TEXT("Expected '%s' to be greater than %f, but it was %f and within equality tolerance %f."), What, Expected, Actual, Tolerance), 1);
		return false;
	}
	if (Actual > Expected)
	{
		return true;
	}
	AddError(FString::Printf(TEXT("Expected '%s' to be greater than %f, but it was %f and outside equality tolerance %f."), What, Expected, Actual, Tolerance), 1);
	return false;
}

bool FAutomationTestBase::TestLessEqual(const TCHAR* What, const float Actual, const float Expected, float Tolerance)
{
	if (Actual < Expected)
	{
		return true;
	}
	if (FMath::IsNearlyEqual(Actual, Expected, Tolerance))
	{
		return true;
	}
	AddError(FString::Printf(TEXT("Expected '%s' to be less than or equal to %f, but it was %f and outside equality tolerance %f."), What, Expected, Actual, Tolerance), 1);
	return false;
}

bool FAutomationTestBase::TestLessEqual(const TCHAR* What, const double Actual, const double Expected, double Tolerance)
{
	if (Actual < Expected)
	{
		return true;
	}
	if (FMath::IsNearlyEqual(Actual, Expected, Tolerance))
	{
		return true;
	}
	AddError(FString::Printf(TEXT("Expected '%s' to be less than or equal to %f, but it was %f and outside equality tolerance %f."), What, Expected, Actual, Tolerance), 1);
	return false;
}

bool FAutomationTestBase::TestGreaterEqual(const TCHAR* What, const float Actual, const float Expected, float Tolerance)
{
	if (Actual > Expected)
	{
		return true;
	}
	if (FMath::IsNearlyEqual(Actual, Expected, Tolerance))
	{
		return true;
	}
	AddError(FString::Printf(TEXT("Expected '%s' to be greater than or equal to %f, but it was %f and outside equality tolerance %f."), What, Expected, Actual, Tolerance), 1);
	return false;
}

bool FAutomationTestBase::TestGreaterEqual(const TCHAR* What, const double Actual, const double Expected, double Tolerance)
{
	if (Actual > Expected)
	{
		return true;
	}
	if (FMath::IsNearlyEqual(Actual, Expected, Tolerance))
	{
		return true;
	}
	AddError(FString::Printf(TEXT("Expected '%s' to be greater than or equal to %f, but it was %f and outside equality tolerance %f."), What, Expected, Actual, Tolerance), 1);
	return false;
}

bool FAutomationTestBase::TestBetweenExclusive(const TCHAR* What, const int32 Actual, const int32 ExpectedLow, const int32 ExpectedHigh)
{
	if (Actual > ExpectedLow && Actual < ExpectedHigh)
	{
		return true;
	}
	AddError(FString::Printf(TEXT("Expected '%s' to be between %d and %d exclusive, but it was %d."), What, ExpectedLow, ExpectedHigh, Actual), 1);
	return false;
}

bool FAutomationTestBase::TestBetweenExclusive(const TCHAR* What, const int64 Actual, const int64 ExpectedLow, const int64 ExpectedHigh)
{
	if (Actual > ExpectedLow && Actual < ExpectedHigh)
	{
		return true;
	}
	AddError(FString::Printf(TEXT("Expected '%s' to be between %" PRId64 " and %" PRId64 " exclusive, but it was %" PRId64 "."), What, ExpectedLow, ExpectedHigh, Actual), 1);
	return false;
}

bool FAutomationTestBase::TestBetweenInclusive(const TCHAR* What, const int32 Actual, const int32 ExpectedLow, const int32 ExpectedHigh)
{
	if (Actual >= ExpectedLow && Actual <= ExpectedHigh)
	{
		return true;
	}
	AddError(FString::Printf(TEXT("Expected '%s' to be between %d and %d inclusive, but it was %d."), What, ExpectedLow, ExpectedHigh, Actual), 1);
	return false;
}

bool FAutomationTestBase::TestBetweenInclusive(const TCHAR* What, const int64 Actual, const int64 ExpectedLow, const int64 ExpectedHigh)
{
	if (Actual >= ExpectedLow && Actual <= ExpectedHigh)
	{
		return true;
	}
	AddError(FString::Printf(TEXT("Expected '%s' to be between %" PRId64 " and %" PRId64 " inclusive, but it was %" PRId64 "."), What, ExpectedLow, ExpectedHigh, Actual), 1);
	return false;
}

bool FAutomationTestBase::TestFalse(const TCHAR* What, bool Value)
{
	if (Value)
	{
		AddError(FString::Printf(TEXT("Expected '%s' to be false."), What), 1);
		return false;
	}
	return true;
}

bool FAutomationTestBase::TestTrue(const TCHAR* What, bool Value)
{
	if (!Value)
	{
		AddError(FString::Printf(TEXT("Expected '%s' to be true."), What), 1);
		return false;
	}
	return true;
}

bool FAutomationTestBase::TestNull(const TCHAR* What, const void* Pointer)
{
	if ( Pointer != nullptr )
	{
		AddError(FString::Printf(TEXT("Expected '%s' to be null."), What), 1);
		return false;
	}
	return true;
}

bool FAutomationTestBase::IsExpectedMessage(
	const FString& Message,
	const ELogVerbosity::Type& Verbosity)
{
	UE::TReadScopeLock Lock(ActionCS);
	for (FAutomationExpectedMessage& ExpectedMessage : ExpectedMessages)
	{
		// Maintains previous behavior: Adjust so that error and fatal messages are tested against when the input verbosity is "Warning"
		// Similarly, any message above warning should be considered an "info" message
		const ELogVerbosity::Type AdjustedMessageVerbosity =
			ExpectedMessage.Verbosity <= ELogVerbosity::Warning 
			? ELogVerbosity::Warning
			: ELogVerbosity::VeryVerbose;

		// Compare the incoming message verbosity with the expected verbosity,
		if (LogCategoryMatchesSeverityInclusive(Verbosity, AdjustedMessageVerbosity) && ExpectedMessage.Matches(Message))
		{
			return true;
		}
	}

	return false;
}
