// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AxisDisplayInfo.h"
#include "Math/IntVector.h"

#include "Features/IModularFeatures.h"
#include "Internationalization/Internationalization.h"

#define LOCTEXT_NAMESPACE "AxisDisplayInfo"

EAxisList::Type AxisDisplayInfo::GetAxisDisplayCoordinateSystem()
{
	if (IModularFeatures::Get().IsModularFeatureAvailable(IAxisDisplayInfo::GetModularFeatureName()))
	{
		IAxisDisplayInfo& DisplayInfo = IModularFeatures::Get().GetModularFeature<IAxisDisplayInfo>(IAxisDisplayInfo::GetModularFeatureName());
		return DisplayInfo.GetAxisDisplayCoordinateSystem();
	}

	return EAxisList::XYZ;
}

FText AxisDisplayInfo::GetAxisDisplayName(EAxisList::Type Axis)
{
	if (IModularFeatures::Get().IsModularFeatureAvailable(IAxisDisplayInfo::GetModularFeatureName()))
	{
		IAxisDisplayInfo& DisplayInfo = IModularFeatures::Get().GetModularFeature<IAxisDisplayInfo>(IAxisDisplayInfo::GetModularFeatureName());
		return DisplayInfo.GetAxisDisplayName(Axis);
	}

	if (Axis == EAxisList::X || Axis == EAxisList::Forward)
	{
		return LOCTEXT("XDisplayName", "X");
	}
	else if (Axis == EAxisList::Y || Axis == EAxisList::Left)
	{
		return LOCTEXT("YDisplayName", "Y");
	}
	else if (Axis == EAxisList::Z || Axis == EAxisList::Up)
	{
		return LOCTEXT("ZDisplayName", "Z");
	}

	ensureMsgf(false, TEXT("Unsupported Axis: %d"), Axis);

	return LOCTEXT("UnsupportedDisplayName", "Unsupported");
}

FText AxisDisplayInfo::GetAxisDisplayNameShort(EAxisList::Type Axis)
{
	if (IModularFeatures::Get().IsModularFeatureAvailable(IAxisDisplayInfo::GetModularFeatureName()))
	{
		IAxisDisplayInfo& DisplayInfo = IModularFeatures::Get().GetModularFeature<IAxisDisplayInfo>(IAxisDisplayInfo::GetModularFeatureName());
		return DisplayInfo.GetAxisDisplayNameShort(Axis);
	}

	if (Axis == EAxisList::X)
	{
		return LOCTEXT("XDisplayNameShort", "X");
	}
	else if (Axis == EAxisList::Y)
	{
		return LOCTEXT("YDisplayNameShort", "Y");
	}
	else if (Axis == EAxisList::Z)
	{
		return LOCTEXT("ZDisplayNameShort", "Z");
	}
	else if (Axis == EAxisList::Forward)
	{
		return LOCTEXT("ForwardDisplayNameShort", "Forward");
	}
	else if (Axis == EAxisList::Left)
	{
		return LOCTEXT("LeftDisplayNameShort", "Left");
	}
	else if (Axis == EAxisList::Up)
	{
		return LOCTEXT("UpDisplayNameShort", "Up");
	}

	ensureMsgf(false, TEXT("Unsupported Axis: %d"), Axis);

	return LOCTEXT("UnsupportedGetAxisDisplayNameShort", "?");
}

FText AxisDisplayInfo::GetAxisToolTip(EAxisList::Type Axis)
{
	if (IModularFeatures::Get().IsModularFeatureAvailable(IAxisDisplayInfo::GetModularFeatureName()))
	{
		IAxisDisplayInfo& DisplayInfo = IModularFeatures::Get().GetModularFeature<IAxisDisplayInfo>(IAxisDisplayInfo::GetModularFeatureName());
		return DisplayInfo.GetAxisToolTip(Axis);
	}
	return LOCTEXT("UnsupportedDisplayName", "Unsupported");
}

FLinearColor AxisDisplayInfo::GetAxisColor(EAxisList::Type Axis)
{
	if (IModularFeatures::Get().IsModularFeatureAvailable(IAxisDisplayInfo::GetModularFeatureName()))
	{
		IAxisDisplayInfo& DisplayInfo = IModularFeatures::Get().GetModularFeature<IAxisDisplayInfo>(IAxisDisplayInfo::GetModularFeatureName());
		return DisplayInfo.GetAxisColor(Axis);
	}

	if (Axis == EAxisList::X || Axis == EAxisList::Forward)
	{
		return FLinearColor(0.594f, 0.0197f, 0.0f);
	}
	else if (Axis == EAxisList::Y || Axis == EAxisList::Left)
	{
		return FLinearColor(0.1349f, 0.3959f, 0.0f);
	}
	else if (Axis == EAxisList::Z || Axis == EAxisList::Up)
	{
		return FLinearColor(0.0251f, 0.207f, 0.85f);
	}

	ensureMsgf(false, TEXT("Unsupported Axis: %d"), Axis);

	return FLinearColor::Black;
}

bool AxisDisplayInfo::UseForwardRightUpDisplayNames()
{
	if (IModularFeatures::Get().IsModularFeatureAvailable(IAxisDisplayInfo::GetModularFeatureName()))
	{
		IAxisDisplayInfo& DisplayInfo = IModularFeatures::Get().GetModularFeature<IAxisDisplayInfo>(IAxisDisplayInfo::GetModularFeatureName());
		return DisplayInfo.UseForwardRightUpDisplayNames();
	}

	return false;
}

FText AxisDisplayInfo::GetRotationAxisNameShort(EAxisList::Type Axis)
{
	if (IModularFeatures::Get().IsModularFeatureAvailable(IAxisDisplayInfo::GetModularFeatureName()))
	{
		IAxisDisplayInfo& DisplayInfo = IModularFeatures::Get().GetModularFeature<IAxisDisplayInfo>(IAxisDisplayInfo::GetModularFeatureName());
		return DisplayInfo.GetRotationAxisNameShort(Axis);
	}

	if (Axis == EAxisList::X)
	{
		return LOCTEXT("Roll_ToolTipTextFormat", "Roll");
	}
	else if (Axis == EAxisList::Y)
	{
		return LOCTEXT("Pitch_ToolTipTextFormat", "Pitch");
	}
	else if (Axis == EAxisList::Z)
	{
		return LOCTEXT("Yaw_ToolTipTextFormat", "Yaw");
	}
	else if (Axis == EAxisList::Forward)
	{
		return LOCTEXT("Forward_ToolTipTextFormat", "Forward");
	}
	else if (Axis == EAxisList::Left)
	{
		return LOCTEXT("Left_ToolTipTextFormat", "Left");
	}
	else if (Axis == EAxisList::Up)
	{
		return LOCTEXT("Up_ToolTipTextFormat", "Up");
	}

	ensureMsgf(false, TEXT("Unsupported Axis: %d"), Axis);

	return LOCTEXT("UnsupportedRotationAxisNameShort", "?");
}

FText AxisDisplayInfo::GetRotationAxisToolTip(EAxisList::Type Axis)
{
	if (IModularFeatures::Get().IsModularFeatureAvailable(IAxisDisplayInfo::GetModularFeatureName()))
	{
		IAxisDisplayInfo& DisplayInfo = IModularFeatures::Get().GetModularFeature<IAxisDisplayInfo>(IAxisDisplayInfo::GetModularFeatureName());
		return DisplayInfo.GetRotationAxisToolTip(Axis);
	}
	return LOCTEXT("UnsupportedRotationAxisToolTip", "?");
}

FIntVector4 AxisDisplayInfo::GetTransformAxisSwizzle()
{
	if (IModularFeatures::Get().IsModularFeatureAvailable(IAxisDisplayInfo::GetModularFeatureName()))
	{
		IAxisDisplayInfo& DisplayInfo = IModularFeatures::Get().GetModularFeature<IAxisDisplayInfo>(IAxisDisplayInfo::GetModularFeatureName());
		return DisplayInfo.DefaultAxisComponentDisplaySwizzle();
	}
	
	return FIntVector4(0, 1, 2, 3);
}

#undef LOCTEXT_NAMESPACE
