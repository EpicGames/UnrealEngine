// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/Base64.h"
#include "Containers/StringConv.h"
#include "Misc/AssertionMacros.h"

namespace
{
	/** The table used to encode a 6 bit value as an ascii character using Table 1 from RFC 4648 ("base64") */
	static constexpr uint8 GBase64EncodingAlphabet[64] =
	{
		'A', 'B', 'C', 'D', 'E', 'F', 'G', 'H', 'I', 'J', 'K', 'L', 'M', 'N', 'O', 'P',
		'Q', 'R', 'S', 'T', 'U', 'V', 'W', 'X', 'Y', 'Z', 'a', 'b', 'c', 'd', 'e', 'f',
		'g', 'h', 'i', 'j', 'k', 'l', 'm', 'n', 'o', 'p', 'q', 'r', 's', 't', 'u', 'v',
		'w', 'x', 'y', 'z', '0', '1', '2', '3', '4', '5', '6', '7', '8', '9', '+', '/'
	};
	/** The table used to encode a 6 bit value as an ascii character using Table 2 from RFC 4648 ("base64url") */
	static constexpr uint8 GBase64UrlSafeEncodingAlphabet[64] =
	{
		'A', 'B', 'C', 'D', 'E', 'F', 'G', 'H', 'I', 'J', 'K', 'L', 'M', 'N', 'O', 'P',
		'Q', 'R', 'S', 'T', 'U', 'V', 'W', 'X', 'Y', 'Z', 'a', 'b', 'c', 'd', 'e', 'f',
		'g', 'h', 'i', 'j', 'k', 'l', 'm', 'n', 'o', 'p', 'q', 'r', 's', 't', 'u', 'v',
		'w', 'x', 'y', 'z', '0', '1', '2', '3', '4', '5', '6', '7', '8', '9', '-', '_'
	};

	/** The table used to convert an ascii character into a 6 bit value using Table 1 from RFC 4648 ("base64") */
	static constexpr uint8 GBase64DecodingAlphabet[128] =
	{
		0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, // 0x00-0x0f
		0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, // 0x10-0x1f
		0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0x3E, 0xFF, 0xFF, 0xFF, 0x3F, // 0x20-0x2f
		0x34, 0x35, 0x36, 0x37, 0x38, 0x39, 0x3A, 0x3B, 0x3C, 0x3D, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, // 0x30-0x3f
		0xFF, 0x00, 0x01, 0x02, 0x03, 0x04, 0x05, 0x06, 0x07, 0x08, 0x09, 0x0A, 0x0B, 0x0C, 0x0D, 0x0E, // 0x40-0x4f
		0x0F, 0x10, 0x11, 0x12, 0x13, 0x14, 0x15, 0x16, 0x17, 0x18, 0x19, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, // 0x50-0x5f
		0xFF, 0x1A, 0x1B, 0x1C, 0x1D, 0x1E, 0x1F, 0x20, 0x21, 0x22, 0x23, 0x24, 0x25, 0x26, 0x27, 0x28, // 0x60-0x6f
		0x29, 0x2A, 0x2B, 0x2C, 0x2D, 0x2E, 0x2F, 0x30, 0x31, 0x32, 0x33, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF  // 0x70-0x7f
	};

	/** The table used to convert an ascii character into a 6 bit value using Table 2 from RFC 4648 ("base64url") */
	static constexpr uint8 GBase64UrlSafeDecodingAlphabet[128] =
	{
		0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, // 0x00-0x0f
		0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, // 0x10-0x1f
		0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0x3E, 0xFF, 0xFF, // 0x20-0x2f
		0x34, 0x35, 0x36, 0x37, 0x38, 0x39, 0x3A, 0x3B, 0x3C, 0x3D, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, // 0x30-0x3f
		0xFF, 0x00, 0x01, 0x02, 0x03, 0x04, 0x05, 0x06, 0x07, 0x08, 0x09, 0x0A, 0x0B, 0x0C, 0x0D, 0x0E, // 0x40-0x4f
		0x0F, 0x10, 0x11, 0x12, 0x13, 0x14, 0x15, 0x16, 0x17, 0x18, 0x19, 0xFF, 0xFF, 0xFF, 0xFF, 0x3F, // 0x50-0x5f
		0xFF, 0x1A, 0x1B, 0x1C, 0x1D, 0x1E, 0x1F, 0x20, 0x21, 0x22, 0x23, 0x24, 0x25, 0x26, 0x27, 0x28, // 0x60-0x6f
		0x29, 0x2A, 0x2B, 0x2C, 0x2D, 0x2E, 0x2F, 0x30, 0x31, 0x32, 0x33, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF  // 0x70-0x7f
	};
}

FString FBase64::Encode(const FString& Source, EBase64Mode Mode)
{
	auto Conv = StrCast<UTF8CHAR>(*Source, Source.Len());
	return Encode((uint8*)Conv.Get(), Conv.Length(), Mode);
}

FString FBase64::Encode(const TArray<uint8>& Source, EBase64Mode Mode)
{
	return Encode((uint8*)Source.GetData(), Source.Num(), Mode);
}

FString FBase64::Encode(const uint8* Source, uint32 Length, EBase64Mode Mode)
{
	uint32 ExpectedLength = GetEncodedDataSize(Length);

	FString OutBuffer;

	TArray<TCHAR, FString::AllocatorType>& OutCharArray = OutBuffer.GetCharArray();
	OutCharArray.SetNum(ExpectedLength + 1);
	Encode(Source, Length, OutCharArray.GetData(), Mode);

	return OutBuffer;
}

template<typename CharType> uint32 FBase64::Encode(const uint8* Source, uint32 Length, CharType* Dest, EBase64Mode Mode)
{
	check(Mode == EBase64Mode::Standard || Mode == EBase64Mode::UrlSafe);

	const uint8* const EncodingAlphabet = (Mode == EBase64Mode::UrlSafe ? GBase64UrlSafeEncodingAlphabet : GBase64EncodingAlphabet);

	const uint32 PaddingLength = Length % 3;
	const uint32 ExpectedLength = GetEncodedDataSize(Length);
	CharType* EncodedBytes = Dest + ExpectedLength;

	// Add a null terminator
	*EncodedBytes = CHARTEXT(CharType, '\0');
	
	if (Length == 0)
	{
		return 0;
	}

	const uint8* ReversedSource = Source + Length;

	// Since this algorithm operates on blocks, we may need to pad the last chunks
	if (PaddingLength > 0)
	{
		EncodedBytes -= 4;

		uint8 A = 0;
		uint8 B = 0;
		uint8 C = 0;
		// Grab the second character if it is a 2 uint8 finish
		if (PaddingLength == 2)
		{
			B = *--ReversedSource;
		}
		A = *--ReversedSource;

		uint32 ByteTriplet = A << 16 | B << 8 | C;
		// Pad with = to make a 4 uint8 chunk
		EncodedBytes[3] = CHARTEXT(CharType, '=');
		ByteTriplet >>= 6;
		// If there's only one 1 uint8 left in the source, then you need 2 pad chars
		if (PaddingLength == 1)
		{
			EncodedBytes[2] = CHARTEXT(CharType, '=');
		}
		else
		{
			EncodedBytes[2] = (CharType)EncodingAlphabet[ByteTriplet & 0x3F];
		}
		// Now encode the remaining bits the same way
		ByteTriplet >>= 6;
		EncodedBytes[1] = (CharType)EncodingAlphabet[ByteTriplet & 0x3F];
		ByteTriplet >>= 6;
		EncodedBytes[0] = (CharType)EncodingAlphabet[ByteTriplet & 0x3F];
	}
	
	Length -= PaddingLength;

	// Loop through the buffer converting 3 bytes of binary data at a time
	while (Length > 0)
	{
		EncodedBytes -= 4;

		uint8 C = *--ReversedSource;
		uint8 B = *--ReversedSource;
		uint8 A = *--ReversedSource;
		Length -= 3;

		// The algorithm takes 24 bits of data (3 bytes) and breaks it into 4 6bit chunks represented as ascii
		uint32 ByteTriplet = A << 16 | B << 8 | C;

		// Use the 6bit block to find the representation ascii character for it
		EncodedBytes[3] = (CharType)EncodingAlphabet[ByteTriplet & 0x3F];
		ByteTriplet >>= 6;
		EncodedBytes[2] = (CharType)EncodingAlphabet[ByteTriplet & 0x3F];
		ByteTriplet >>= 6;
		EncodedBytes[1] = (CharType)EncodingAlphabet[ByteTriplet & 0x3F];
		ByteTriplet >>= 6;
		EncodedBytes[0] = (CharType)EncodingAlphabet[ByteTriplet & 0x3F];
	}

	verify(EncodedBytes == Dest);

	return ExpectedLength;
}

template CORE_API uint32 FBase64::Encode<ANSICHAR>(const uint8* Source, uint32 Length, ANSICHAR* Dest, EBase64Mode Mode);
template CORE_API uint32 FBase64::Encode<WIDECHAR>(const uint8* Source, uint32 Length, WIDECHAR* Dest, EBase64Mode Mode);
template CORE_API uint32 FBase64::Encode<UTF8CHAR>(const uint8* Source, uint32 Length, UTF8CHAR* Dest, EBase64Mode Mode);

bool FBase64::Decode(const FString& Source, FString& OutDest, EBase64Mode Mode)
{
	uint32 ExpectedLength = GetDecodedDataSize(Source);

	TArray<UTF8CHAR> TempDest;
	TempDest.AddUninitialized(ExpectedLength);
	if (!Decode(*Source, Source.Len(), (uint8*)TempDest.GetData(), Mode))
	{
		return false;
	}
	OutDest = FString::ConstructFromPtrSize(TempDest.GetData(), ExpectedLength);
	return true;
}

bool FBase64::Decode(const FString& Source, TArray<uint8>& OutDest, EBase64Mode Mode)
{
	OutDest.SetNum(GetDecodedDataSize(Source));
	return Decode(*Source, Source.Len(), OutDest.GetData(), Mode);
}

template<typename CharType> 
bool FBase64::Decode(const CharType* Source, uint32 Length, uint8* Dest, EBase64Mode Mode)
{
	check(Mode == EBase64Mode::Standard || Mode == EBase64Mode::UrlSafe);

	// Remove the trailing '=' characters, so we can handle padded and non-padded input the same
	while (Length > 0 && Source[Length - 1] == '=')
	{
		Length--;
	}

	// Make sure the length is valid. Only lengths modulo 4 of 0, 2, and 3 are valid.
	if ((Length & 3) == 1)
	{
		return false;
	}

	const uint8* const DecodingAlphabet = (Mode == EBase64Mode::UrlSafe ? GBase64UrlSafeDecodingAlphabet : GBase64DecodingAlphabet);

	// Convert all the full chunks of data
	for (; Length >= 4; Length -= 4)
	{
		// Decode the next 4 BYTEs
		uint32 OriginalTriplet = 0;
		for (int32 Index = 0; Index < 4; Index++)
		{
			uint32 SourceChar = (uint32)*Source++;
			if (SourceChar >= 128)
			{
				return false;
			}
			uint8 DecodedValue = DecodingAlphabet[SourceChar];
			if (DecodedValue == 0xFF)
			{
				return false;
			}
			OriginalTriplet = (OriginalTriplet << 6) | DecodedValue;
		}

		// Now we can tear the uint32 into bytes
		Dest[2] = OriginalTriplet & 0xFF;
		OriginalTriplet >>= 8;
		Dest[1] = OriginalTriplet & 0xFF;
		OriginalTriplet >>= 8;
		Dest[0] = OriginalTriplet & 0xFF;

		// Move to the next output chunk
		Dest += 3;
	}

	// Convert the last chunk of data
	if (Length > 0)
	{
		// Decode the next 4 BYTEs, or up to the end of the input buffer
		uint32 OriginalTriplet = 0;
		for (uint32 Index = 0; Index < Length; Index++)
		{
			uint32 SourceChar = (uint32)*Source++;
			if (SourceChar >= 128)
			{
				return false;
			}
			uint8 DecodedValue = DecodingAlphabet[SourceChar];
			if (DecodedValue == 0xFF)
			{
				return false;
			}
			OriginalTriplet = (OriginalTriplet << 6) | DecodedValue;
		}
		for (int32 Index = Length; Index < 4; Index++)
		{
			OriginalTriplet = (OriginalTriplet << 6);
		}

		// Now we can tear the uint32 into bytes
		OriginalTriplet >>= 8;
		if (Length >= 3)
		{
			Dest[1] = OriginalTriplet & 0xFF;
		}
		OriginalTriplet >>= 8;
		Dest[0] = OriginalTriplet & 0xFF;
	}
	return true;
}

template CORE_API bool FBase64::Decode<ANSICHAR>(const ANSICHAR* Source, uint32 Length, uint8* Dest, EBase64Mode Mode);
template CORE_API bool FBase64::Decode<WIDECHAR>(const WIDECHAR* Source, uint32 Length, uint8* Dest, EBase64Mode Mode);
template CORE_API bool FBase64::Decode<UTF8CHAR>(const UTF8CHAR* Source, uint32 Length, uint8* Dest, EBase64Mode Mode);

uint32 FBase64::GetDecodedDataSize(const FString& Source)
{
	return GetDecodedDataSize(*Source, Source.Len());
}

template<typename CharType> uint32 FBase64::GetDecodedDataSize(const CharType* Source, uint32 Length)
{
	uint32 NumBytes = 0;
	if (Length > 0)
	{
		// Get the source length without the trailing padding characters
		while (Length > 0 && Source[Length - 1] == '=')
		{
			Length--;
		}

		// Get the lower bound for number of bytes, excluding the last partial chunk
		NumBytes += (Length / 4) * 3;
		if ((Length & 3) == 3)
		{
			NumBytes += 2;
		}
		if ((Length & 3) == 2)
		{
			NumBytes++;
		}
	}
	return NumBytes;
}

template CORE_API uint32 FBase64::GetDecodedDataSize(const ANSICHAR* Source, uint32 Length);
template CORE_API uint32 FBase64::GetDecodedDataSize(const WIDECHAR* Source, uint32 Length);
template CORE_API uint32 FBase64::GetDecodedDataSize(const UTF8CHAR* Source, uint32 Length);
