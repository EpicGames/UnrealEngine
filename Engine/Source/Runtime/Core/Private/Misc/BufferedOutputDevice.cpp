// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/BufferedOutputDevice.h"

#include "Misc/ScopeLock.h"
#include "Templates/UnrealTemplate.h"

FBufferedOutputDevice::FBufferedOutputDevice() = default; // Here to satisfy modular builds on some old platforms
FBufferedOutputDevice::~FBufferedOutputDevice() = default; // Here to satisfy modular builds on some old platforms

void FBufferedOutputDevice::Serialize(const TCHAR* InData, ELogVerbosity::Type Verbosity, const FName& Category)
{
	if (Verbosity > FilterLevel)
	{
		return;
	}

	FScopeLock ScopeLock(&SynchronizationObject);
	BufferedLines.Emplace(InData, Category, Verbosity);
}

void FBufferedOutputDevice::GetContents(TArray<FBufferedLine>& DestBuffer)
{
	FScopeLock ScopeLock(&SynchronizationObject);
	DestBuffer = MoveTemp(BufferedLines);
}

void FBufferedOutputDevice::RedirectTo(FOutputDevice& Ar)
{
	FScopeLock ScopeLock(&SynchronizationObject);
	for (const FBufferedLine& BufferedLine : BufferedLines)
	{
		Ar.Serialize(BufferedLine.Data.Get(), BufferedLine.Verbosity, BufferedLine.Category);
	}
}
