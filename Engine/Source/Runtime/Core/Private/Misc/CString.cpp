// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/CString.h"
#include "Containers/StringConv.h"
#include "Internationalization/Text.h"

// 4 lines of 64 chars each, plus a null terminator
template <>
CORE_API const ANSICHAR TCStringSpcHelper<ANSICHAR>::SpcArray[MAX_SPACES + 1] =
	"                                                                "
	"                                                                "
	"                                                                "
	"                                                               ";

template <>
CORE_API const WIDECHAR TCStringSpcHelper<WIDECHAR>::SpcArray[MAX_SPACES + 1] = WIDETEXT(
	"                                                                "
	"                                                                "
	"                                                                "
	"                                                               ");

template <>
CORE_API const UTF8CHAR TCStringSpcHelper<UTF8CHAR>::SpcArray[MAX_SPACES + 1] = {
	UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '),
	UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '),
	UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '),
	UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '),
	UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '),
	UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '),
	UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '),
	UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '),

	UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '),
	UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '),
	UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '),
	UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '),
	UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '),
	UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '),
	UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '),
	UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '),

	UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '),
	UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '),
	UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '),
	UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '),
	UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '),
	UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '),
	UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '),
	UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '),

	UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '),
	UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '),
	UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '),
	UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '),
	UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '),
	UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '),
	UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '),
	UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT(' '), UTF8TEXT('\0')
};

template <>
CORE_API const ANSICHAR TCStringSpcHelper<ANSICHAR>::TabArray[MAX_TABS + 1] = 
	"\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t"
	"\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t"
	"\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t"
	"\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t";

template <>
CORE_API const WIDECHAR TCStringSpcHelper<WIDECHAR>::TabArray[MAX_TABS + 1] = WIDETEXT(
	"\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t"
	"\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t"
	"\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t"
	"\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t\t");

template <>
CORE_API const UTF8CHAR TCStringSpcHelper<UTF8CHAR>::TabArray[MAX_TABS + 1] = {
	UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'),
	UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'),
	UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'),
	UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'),
	UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'),
	UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'),
	UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'),
	UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'),

	UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'),
	UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'),
	UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'),
	UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'),
	UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'),
	UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'),
	UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'),
	UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'),

	UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'),
	UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'),
	UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'),
	UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'),
	UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'),
	UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'),
	UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'),
	UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'),

	UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'),
	UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'),
	UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'),
	UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'),
	UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'),
	UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'),
	UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'),
	UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\t'), UTF8TEXT('\0')
};

bool FToBoolHelper::FromCStringAnsi( const ANSICHAR* String )
{
#if PLATFORM_TCHAR_IS_UTF8CHAR
	return FToBoolHelper::FromCStringUtf8( StringCast<UTF8CHAR>(String).Get() );
#else
	return FToBoolHelper::FromCStringWide( StringCast<WIDECHAR>(String).Get() );
#endif
}

bool FToBoolHelper::FromCStringWide( const WIDECHAR* String )
{
#if PLATFORM_TCHAR_IS_UTF8CHAR
	return FToBoolHelper::FromCStringUtf8( StringCast<UTF8CHAR>(String).Get() );
#else
	const FCoreTexts& CoreTexts = FCoreTexts::Get();

	if (
			FCStringWide::Stricmp(String, TEXT("True"))==0
		||	FCStringWide::Stricmp(String, TEXT("Yes"))==0
		||	FCStringWide::Stricmp(String, TEXT("On"))==0
		||	FCStringWide::Stricmp(String, *(CoreTexts.True.ToString()))==0
		||	FCStringWide::Stricmp(String, *(CoreTexts.Yes.ToString()))==0)
	{
		return true;
	}
	else if(
			FCStringWide::Stricmp(String, TEXT("False"))==0
		||	FCStringWide::Stricmp(String, TEXT("No"))==0
		||	FCStringWide::Stricmp(String, TEXT("Off"))==0
		||	FCStringWide::Stricmp(String, *(CoreTexts.False.ToString()))==0
		||	FCStringWide::Stricmp(String, *(CoreTexts.No.ToString()))==0)
	{
		return false;
	}
	else
	{
		return FCStringWide::Atoi(String) ? true : false;
	}
#endif
}

bool FToBoolHelper::FromCStringUtf8( const UTF8CHAR* String )
{
#if PLATFORM_TCHAR_IS_UTF8CHAR
	const FCoreTexts& CoreTexts = FCoreTexts::Get();

	if (
			FCStringUtf8::Stricmp(String, TEXT("True"))==0
		||	FCStringUtf8::Stricmp(String, TEXT("Yes"))==0
		||	FCStringUtf8::Stricmp(String, TEXT("On"))==0
		||	FCStringUtf8::Stricmp(String, *(CoreTexts.True.ToString()))==0
		||	FCStringUtf8::Stricmp(String, *(CoreTexts.Yes.ToString()))==0)
	{
		return true;
	}
	else if(
			FCStringUtf8::Stricmp(String, TEXT("False"))==0
		||	FCStringUtf8::Stricmp(String, TEXT("No"))==0
		||	FCStringUtf8::Stricmp(String, TEXT("Off"))==0
		||	FCStringUtf8::Stricmp(String, *(CoreTexts.False.ToString()))==0
		||	FCStringUtf8::Stricmp(String, *(CoreTexts.No.ToString()))==0)
	{
		return false;
	}
	else
	{
		return FCStringUtf8::Atoi(String) ? true : false;
	}
#else
	return FToBoolHelper::FromCStringWide( StringCast<WIDECHAR>(String).Get() );
#endif
}

int32 UE::Core::Private::Strlen32(const UTF32CHAR* String)
{
	int32 Result = 0;
	while (*String)
	{
		++Result;
		++String;
	}
	return Result;
}
