// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/CString.h"

#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

#include "Containers/StringView.h"
#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "Misc/StringBuilder.h"

// This class is a workaround for clang compilers causing error "'va_start' used in function with fixed args" when using it in a lambda in RunTest()
class FCStringGetVarArgsTestBase : public FAutomationTestBase
{
public:
	FCStringGetVarArgsTestBase(const FString& InName, const bool bInComplexTask)
	: FAutomationTestBase(InName, bInComplexTask)
	{
	}

protected:
	void DoTest(const TCHAR* ExpectedOutput, const TCHAR* Format, ...)
	{
		constexpr SIZE_T OutputBufferCharacterCount = 512;
		TCHAR OutputBuffer[OutputBufferCharacterCount];
		va_list ArgPtr;
		va_start(ArgPtr, Format);
		const int32 Result = FCString::GetVarArgs(OutputBuffer, OutputBufferCharacterCount, Format, ArgPtr);
		va_end(ArgPtr);

		if (Result < 0)
		{
			this->AddError(FString::Printf(TEXT("'%s' could not be parsed."), Format));
			return;
		}

		if (FCString::Strcmp(OutputBuffer, ExpectedOutput) != 0)
		{
			this->AddError(FString::Printf(TEXT("'%s' resulted in '%s', expected '%s'."), Format, OutputBuffer, ExpectedOutput));
			return;
		}
	}
};

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FCStringGetVarArgsTest, FCStringGetVarArgsTestBase, "System.Core.Misc.CString.GetVarArgs", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
bool FCStringGetVarArgsTest::RunTest(const FString& Parameters)
{
	DoTest(TEXT("SIZE_T_FMT |18446744073709551615|"), TEXT("SIZE_T_FMT |%" SIZE_T_FMT "|"), SIZE_T(MAX_uint64));
	DoTest(TEXT("SIZE_T_x_FMT |ffffffffffffffff|"), TEXT("SIZE_T_x_FMT |%" SIZE_T_x_FMT "|"), UPTRINT(MAX_uint64));
	DoTest(TEXT("SIZE_T_X_FMT |FFFFFFFFFFFFFFFF|"), TEXT("SIZE_T_X_FMT |%" SIZE_T_X_FMT "|"), UPTRINT(MAX_uint64));

	DoTest(TEXT("SSIZE_T_FMT |-9223372036854775808|"), TEXT("SSIZE_T_FMT |%" SSIZE_T_FMT "|"), SSIZE_T(MIN_int64));
	DoTest(TEXT("SSIZE_T_x_FMT |ffffffffffffffff|"), TEXT("SSIZE_T_x_FMT |%" SSIZE_T_x_FMT "|"), SSIZE_T(-1));
	DoTest(TEXT("SSIZE_T_X_FMT |FFFFFFFFFFFFFFFF|"), TEXT("SSIZE_T_X_FMT |%" SSIZE_T_X_FMT "|"), SSIZE_T(-1));

	DoTest(TEXT("PTRINT_FMT |-9223372036854775808|"), TEXT("PTRINT_FMT |%" PTRINT_FMT "|"), PTRINT(MIN_int64));
	DoTest(TEXT("PTRINT_x_FMT |ffffffffffffffff|"), TEXT("PTRINT_x_FMT |%" PTRINT_x_FMT "|"), PTRINT(-1));
	DoTest(TEXT("PTRINT_X_FMT |FFFFFFFFFFFFFFFF|"), TEXT("PTRINT_X_FMT |%" PTRINT_X_FMT "|"), PTRINT(-1));

	DoTest(TEXT("UPTRINT_FMT |18446744073709551615|"), TEXT("UPTRINT_FMT |%" UPTRINT_FMT "|"), UPTRINT(MAX_uint64));
	DoTest(TEXT("UPTRINT_x_FMT |ffffffffffffffff|"), TEXT("UPTRINT_x_FMT |%" UPTRINT_x_FMT "|"), UPTRINT(MAX_uint64));
	DoTest(TEXT("UPTRINT_X_FMT |FFFFFFFFFFFFFFFF|"), TEXT("UPTRINT_X_FMT |%" UPTRINT_X_FMT "|"), UPTRINT(MAX_uint64));

	DoTest(TEXT("INT64_FMT |-9223372036854775808|"), TEXT("INT64_FMT |%" INT64_FMT "|"), MIN_int64);
	DoTest(TEXT("INT64_x_FMT |ffffffffffffffff|"), TEXT("INT64_x_FMT |%" INT64_x_FMT "|"), int64(-1));
	DoTest(TEXT("INT64_X_FMT |FFFFFFFFFFFFFFFF|"), TEXT("INT64_X_FMT |%" INT64_X_FMT "|"), int64(-1));

	DoTest(TEXT("UINT64_FMT |18446744073709551615|"), TEXT("UINT64_FMT |%" UINT64_FMT "|"), MAX_uint64);
	DoTest(TEXT("UINT64_x_FMT |ffffffffffffffff|"), TEXT("UINT64_x_FMT |%" UINT64_x_FMT "|"), MAX_uint64);
	DoTest(TEXT("UINT64_X_FMT |FFFFFFFFFFFFFFFF|"), TEXT("UINT64_X_FMT |%" UINT64_X_FMT "|"), MAX_uint64);

	DoTest(TEXT("|LEFT                |               RIGHT|     33.33|66.67     |"), TEXT("|%-20s|%20s|%10.2f|%-10.2f|"), TEXT("LEFT"), TEXT("RIGHT"), 33.333333, 66.666666);

	DoTest(TEXT("Percents|%%%3|"), TEXT("Percents|%%%%%%%d|"), 3);

	DoTest(TEXT("Integer arguments|12345|54321|123ABC|f|99|"), TEXT("Integer arguments|%d|%i|%X|%x|%u|"), 12345, 54321, 0x123AbC, 15, 99);
	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FCStringStrstrTest, "System.Core.Misc.CString.Strstr", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
bool FCStringStrstrTest::RunTest(const FString& Parameters)
{
	auto RunTest = [this](const TCHAR* Search, const TCHAR* Find, int32 ExpectedSensitiveIndex, int32 ExpectedInsensitiveIndex)
	{
		const TCHAR* ExpectedSensitive = ExpectedSensitiveIndex == INDEX_NONE ? nullptr :
			(Search + ExpectedSensitiveIndex);
		const TCHAR* ExpectedInsensitive = ExpectedInsensitiveIndex == INDEX_NONE ? nullptr :
			(Search + ExpectedInsensitiveIndex);
		if (FCString::Strstr(Search, Find) != ExpectedSensitive)
		{
			AddError(FString::Printf(TEXT("Strstr(\"%s\", \"%s\") did not equal index \"%d\"."),
				Search, Find, ExpectedSensitiveIndex));
		}
		if (FCString::Stristr(Search, Find) != ExpectedInsensitive)
		{
			AddError(FString::Printf(TEXT("Stristr(\"%s\", \"%s\") did not equal index \"%d\"."),
				Search, Find, ExpectedInsensitiveIndex));
		}
	};
	const TCHAR* ABACADAB = TEXT("ABACADAB");

	RunTest(ABACADAB, TEXT("A"), 0, 0);
	RunTest(ABACADAB, TEXT("a"), INDEX_NONE, 0);
	RunTest(ABACADAB, TEXT("BAC"), 1, 1);
	RunTest(ABACADAB, TEXT("BaC"), INDEX_NONE, 1);
	RunTest(ABACADAB, TEXT("BAC"), 1, 1);
	RunTest(ABACADAB, TEXT("BaC"), INDEX_NONE, 1);
	RunTest(ABACADAB, TEXT("DAB"), 5, 5);
	RunTest(ABACADAB, TEXT("dab"), INDEX_NONE, 5);
	RunTest(ABACADAB, ABACADAB, 0, 0);
	RunTest(ABACADAB, TEXT("abacadab"), INDEX_NONE, 0);
	RunTest(ABACADAB, TEXT("F"), INDEX_NONE, INDEX_NONE);
	RunTest(ABACADAB, TEXT("DABZ"), INDEX_NONE, INDEX_NONE);
	RunTest(ABACADAB, TEXT("ABACADABA"), INDEX_NONE, INDEX_NONE);
	RunTest(ABACADAB, TEXT("NoMatchLongerString"), INDEX_NONE, INDEX_NONE);
	RunTest(TEXT(""), TEXT("FindText"), INDEX_NONE, INDEX_NONE);
	RunTest(TEXT(""), TEXT(""), 0, 0);
	RunTest(ABACADAB, TEXT(""), 0, 0);

	// Passing in nullpt r is not allowed by StrStr so we do not test it

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FCStringStrnstrTest, "System.Core.Misc.CString.Strnstr", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
bool FCStringStrnstrTest::RunTest(const FString& Parameters)
{
	auto RunTest = [this](FStringView Search, FStringView Find, int32 ExpectedSensitiveIndex, int32 ExpectedInsensitiveIndex)
	{
		TStringBuilder<128> SearchWithoutNull;
		TStringBuilder<128> FindWithoutNull;
		// TODO: If we could add a way to assert the trailing characters are not read, that would be a better test
		SearchWithoutNull << Search << TEXT("SearchTrailing");
		FindWithoutNull << Find << TEXT("FindTrailing");
		const TCHAR* ExpectedSensitive = ExpectedSensitiveIndex == INDEX_NONE ? nullptr :
			(Search.GetData() + ExpectedSensitiveIndex);
		const TCHAR* ExpectedInsensitive = ExpectedInsensitiveIndex == INDEX_NONE ? nullptr :
			(Search.GetData() + ExpectedInsensitiveIndex);
		const TCHAR* ExpectedSensitiveWithoutNull = ExpectedSensitiveIndex == INDEX_NONE ? nullptr :
			(SearchWithoutNull.GetData() + ExpectedSensitiveIndex);
		const TCHAR* ExpectedInsensitiveWithoutNull = ExpectedInsensitiveIndex == INDEX_NONE ? nullptr :
			(SearchWithoutNull.GetData() + ExpectedInsensitiveIndex);
		if (FCString::Strnstr(Search.GetData(), Search.Len(), Find.GetData(), Find.Len()) != ExpectedSensitive)
		{
			AddError(FString::Printf(TEXT("Strnstr(\"%.*s\", %d, \"%.*s\", %d)\" did not equal index \"%d\"."),
				Search.Len(), Search.GetData(), Search.Len(), Find.Len(), Find.GetData(), Find.Len(), ExpectedSensitiveIndex));
		}
		if (FCString::Strnstr(SearchWithoutNull.GetData(), Search.Len(), FindWithoutNull.GetData(), Find.Len()) != ExpectedSensitiveWithoutNull)
		{
			AddError(FString::Printf(TEXT("Strnstr(\"%.*s\", %d, \"%.*s\", %d)\" did not equal index \"%d\", when embedded in a string without a nullterminator."),
				Search.Len(), Search.GetData(), Search.Len(), Find.Len(), Find.GetData(), Find.Len(), ExpectedSensitiveIndex));
		}
		if (FCString::Strnistr(Search.GetData(), Search.Len(), Find.GetData(), Find.Len()) != ExpectedInsensitive)
		{
			AddError(FString::Printf(TEXT("Strnistr(\"%.*s\", %d, \"%.*s\", %d)\" did not equal index \"%d\"."),
				Search.Len(), Search.GetData(), Search.Len(), Find.Len(), Find.GetData(), Find.Len(), ExpectedInsensitiveIndex));
		}
		if (FCString::Strnistr(SearchWithoutNull.GetData(), Search.Len(), FindWithoutNull.GetData(), Find.Len()) != ExpectedInsensitiveWithoutNull)
		{
			AddError(FString::Printf(TEXT("Strnistr(\"%.*s\", %d, \"%.*s\", %d)\" did not equal index \"%d\", when embedded in a string without a nullterminator."),
				Search.Len(), Search.GetData(), Search.Len(), Find.Len(), Find.GetData(), Find.Len(), ExpectedInsensitiveIndex));
		}
	};
	FStringView ABACADAB(TEXTVIEW("ABACADAB"));

	RunTest(ABACADAB, TEXTVIEW("A"), 0, 0);
	RunTest(ABACADAB, TEXTVIEW("a"), INDEX_NONE, 0);
	RunTest(ABACADAB, TEXTVIEW("BAC"), 1, 1);
	RunTest(ABACADAB, TEXTVIEW("BaC"), INDEX_NONE, 1);
	RunTest(ABACADAB, TEXTVIEW("BAC"), 1, 1);
	RunTest(ABACADAB, TEXTVIEW("BaC"), INDEX_NONE, 1);
	RunTest(ABACADAB, TEXTVIEW("DAB"), 5, 5);
	RunTest(ABACADAB, TEXTVIEW("dab"), INDEX_NONE, 5);
	RunTest(ABACADAB, ABACADAB, 0, 0);
	RunTest(ABACADAB, TEXTVIEW("abacadab"), INDEX_NONE, 0);
	RunTest(ABACADAB, TEXTVIEW("F"), INDEX_NONE, INDEX_NONE);
	RunTest(ABACADAB, TEXTVIEW("DABZ"), INDEX_NONE, INDEX_NONE);
	RunTest(ABACADAB, TEXTVIEW("ABACADABA"), INDEX_NONE, INDEX_NONE);
	RunTest(ABACADAB, TEXTVIEW("NoMatchLongerString"), INDEX_NONE, INDEX_NONE);
	RunTest(TEXTVIEW(""), TEXTVIEW("FindText"), INDEX_NONE, INDEX_NONE);
	RunTest(TEXTVIEW(""), TEXTVIEW(""), 0, 0);
	RunTest(ABACADAB, TEXTVIEW(""), 0, 0);

	// Tests that pass in nullptr
	const TCHAR* NullString = nullptr;
	const TCHAR* EmptyString = TEXT("");
	if (FCString::Strnstr(NullString, 0, NullString, 0) != nullptr ||
		FCString::Strnistr(NullString, 0, NullString, 0) != nullptr)
	{
		AddError(TEXT("Strnstr(nullptr, 0, nullptr, 0) did not equal nullptr."));
	}
	if (FCString::Strnstr(EmptyString, 0, NullString, 0) != EmptyString ||
		FCString::Strnistr(EmptyString, 0, NullString, 0) != EmptyString)
	{
		AddError(TEXT("Strnstr(EmptyString, 0, nullptr, 0) did not equal EmptyString."));
	}
	if (FCString::Strnstr(NullString, 0, EmptyString, 0) != nullptr ||
		FCString::Strnistr(NullString, 0, EmptyString, 0) != nullptr)
	{
		AddError(TEXT("Strnstr(nullptr, 0, EmptyString, 0) did not equal nullptr."));
	}

	// Negative lengths are not allowed so we do not test them.

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FCStringStrcpyTest, "System.Core.Misc.CString.Strcpy", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
bool FCStringStrcpyTest::RunTest(const FString& Parameters)
{
	constexpr int32 BufferLen = 32;
	WIDECHAR WideBuffer[BufferLen];
	UTF8CHAR Utf8Buffer[BufferLen];
	ANSICHAR AnsiBuffer[BufferLen];

	const WIDECHAR* WideTest = WIDETEXT("12345");
	const UTF8CHAR* Utf8Test = UTF8TEXT("12345");
	const ANSICHAR* AnsiTest = "12345";
	constexpr int32 TestLen = 5;

	auto Reset = [&WideBuffer, &Utf8Buffer, &AnsiBuffer, BufferLen]()
		{
			for (int32 n = 0; n < BufferLen; ++n)
			{
				WideBuffer[n] = '%';
				Utf8Buffer[n] = UTF8CHAR('%');
				AnsiBuffer[n] = '%';
			}
		};

	Reset();
	FCStringWide::Strcpy(WideBuffer, WideTest);
	FCStringUtf8::Strcpy(Utf8Buffer, Utf8Test);
	FCStringAnsi::Strcpy(AnsiBuffer, AnsiTest);
	this->TestTrue(TEXT("WideStrcpy"),
		WideBuffer[TestLen] == 0 && FCStringWide::Strcmp(WideTest, WideBuffer) == 0 && WideBuffer[TestLen + 1] == '%');
	this->TestTrue(TEXT("Utf8Strcpy"),
		Utf8Buffer[TestLen] == 0 && FCStringUtf8::Strcmp(Utf8Test, Utf8Buffer) == 0 && Utf8Buffer[TestLen + 1] == '%');
	this->TestTrue(TEXT("AnsiStrcpy"),
		AnsiBuffer[TestLen] == 0 && FCStringAnsi::Strcmp(AnsiTest, AnsiBuffer) == 0 && AnsiBuffer[TestLen + 1] == '%');

	Reset();
	FCStringWide::Strncpy(WideBuffer, WideTest, TestLen + 10);
	FCStringUtf8::Strncpy(Utf8Buffer, Utf8Test, TestLen + 10);
	FCStringAnsi::Strncpy(AnsiBuffer, AnsiTest, TestLen + 10);
	this->TestTrue(TEXT("WideStrncpyTestLenPlus10"),
		WideBuffer[TestLen] == 0 && FCStringWide::Strcmp(WideTest, WideBuffer) == 0 && WideBuffer[TestLen + 10] == '%');
	this->TestTrue(TEXT("Utf8StrncpyTestLenPlus10"),
		Utf8Buffer[TestLen] == 0 && FCStringUtf8::Strcmp(Utf8Test, Utf8Buffer) == 0 && Utf8Buffer[TestLen + 10] == '%');
	this->TestTrue(TEXT("AnsiStrncpyTestLenPlus10"),
		AnsiBuffer[TestLen] == 0 && FCStringAnsi::Strcmp(AnsiTest, AnsiBuffer) == 0 && AnsiBuffer[TestLen + 10] == '%');

	Reset();
	FCStringWide::Strncpy(WideBuffer, WideTest, TestLen + 1);
	FCStringUtf8::Strncpy(Utf8Buffer, Utf8Test, TestLen + 1);
	FCStringAnsi::Strncpy(AnsiBuffer, AnsiTest, TestLen + 1);
	this->TestTrue(TEXT("WideStrncpyTestLenPlus1"),
		WideBuffer[TestLen] == 0 && FCStringWide::Strcmp(WideTest, WideBuffer) == 0 && WideBuffer[TestLen + 1] == '%');
	this->TestTrue(TEXT("Utf8StrncpyTestLenPlus1"),
		Utf8Buffer[TestLen] == 0 && FCStringUtf8::Strcmp(Utf8Test, Utf8Buffer) == 0 && Utf8Buffer[TestLen + 1] == '%');
	this->TestTrue(TEXT("AnsiStrncpyTestLenPlus1"),
		AnsiBuffer[TestLen] == 0 && FCStringAnsi::Strcmp(AnsiTest, AnsiBuffer) == 0 && AnsiBuffer[TestLen + 1] == '%');

	Reset();
	FCStringWide::Strncpy(WideBuffer, WideTest, TestLen);
	FCStringUtf8::Strncpy(Utf8Buffer, Utf8Test, TestLen);
	FCStringAnsi::Strncpy(AnsiBuffer, AnsiTest, TestLen);
	this->TestTrue(TEXT("WideStrncpyTestLen"),
		WideBuffer[TestLen - 1] == 0 && WideBuffer[TestLen] == '%' && FCStringWide::Strncmp(WideTest, WideBuffer, TestLen - 1) == 0 && WideBuffer[TestLen + 1] == '%');
	this->TestTrue(TEXT("Utf8StrncpyTestLen"),
		Utf8Buffer[TestLen - 1] == 0 && Utf8Buffer[TestLen] == '%' && FCStringUtf8::Strncmp(Utf8Test, Utf8Buffer, TestLen - 1) == 0 && Utf8Buffer[TestLen + 1] == '%');
	this->TestTrue(TEXT("AnsiStrncpyTestLen"),
		AnsiBuffer[TestLen - 1] == 0 && AnsiBuffer[TestLen] == '%' && FCStringAnsi::Strncmp(AnsiTest, AnsiBuffer, TestLen - 1) == 0 && AnsiBuffer[TestLen + 1] == '%');

	Reset();
	FCStringWide::Strncpy(WideBuffer, WideTest, TestLen - 1);
	FCStringUtf8::Strncpy(Utf8Buffer, Utf8Test, TestLen - 1);
	FCStringAnsi::Strncpy(AnsiBuffer, AnsiTest, TestLen - 1);
	this->TestTrue(TEXT("WideStrncpyTestLenMinus1"),
		WideBuffer[TestLen - 2] == 0 && WideBuffer[TestLen - 1] == '%' && FCStringWide::Strncmp(WideTest, WideBuffer, TestLen - 2) == 0 && WideBuffer[TestLen + 1] == '%');
	this->TestTrue(TEXT("Utf8StrncpyTestLenMinus1"),
		Utf8Buffer[TestLen - 2] == 0 && Utf8Buffer[TestLen - 1] == '%' && FCStringUtf8::Strncmp(Utf8Test, Utf8Buffer, TestLen - 2) == 0 && Utf8Buffer[TestLen + 1] == '%');
	this->TestTrue(TEXT("AnsiStrncpyTestLenMinus1"),
		AnsiBuffer[TestLen - 2] == 0 && AnsiBuffer[TestLen - 1] == '%' && FCStringAnsi::Strncmp(AnsiTest, AnsiBuffer, TestLen - 2) == 0 && AnsiBuffer[TestLen + 1] == '%');

	Reset();
	FCStringWide::Strncpy(WideBuffer, WideTest, 2);
	FCStringUtf8::Strncpy(Utf8Buffer, Utf8Test, 2);
	FCStringAnsi::Strncpy(AnsiBuffer, AnsiTest, 2);
	this->TestTrue(TEXT("WideStrncpyTwoLen"),
		WideBuffer[0] == WideTest[0] && WideBuffer[1] == 0 && WideBuffer[2] == '%' && WideBuffer[TestLen] == '%');
	this->TestTrue(TEXT("Utf8StrncpyTwoLen"),
		Utf8Buffer[0] == Utf8Test[0] && Utf8Buffer[1] == 0 && Utf8Buffer[2] == '%' && Utf8Buffer[TestLen] == '%');
	this->TestTrue(TEXT("AnsiStrncpyTwoLen"),
		AnsiBuffer[0] == AnsiTest[0] && AnsiBuffer[1] == 0 && AnsiBuffer[2] == '%' && AnsiBuffer[TestLen] == '%');

	Reset();
	FCStringWide::Strncpy(WideBuffer, WideTest, 1);
	FCStringUtf8::Strncpy(Utf8Buffer, Utf8Test, 1);
	FCStringAnsi::Strncpy(AnsiBuffer, AnsiTest, 1);
	this->TestTrue(TEXT("WideStrncpyOneLen"),
		WideBuffer[0] == 0 && WideBuffer[1] == '%' && WideBuffer[TestLen] == '%');
	this->TestTrue(TEXT("Utf8StrncpyOneLen"),
		Utf8Buffer[0] == 0 && Utf8Buffer[1] == '%' && Utf8Buffer[TestLen] == '%');
	this->TestTrue(TEXT("AnsiStrncpyOneLen"),
		AnsiBuffer[0] == 0 && AnsiBuffer[1] == '%' && AnsiBuffer[TestLen] == '%');

	// ZeroLen Strncpy is undefined
	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FCStringStrcatTest, "System.Core.Misc.CString.Strcat", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
bool FCStringStrcatTest::RunTest(const FString& Parameters)
{
	constexpr int32 BufferLen = 32;
	WIDECHAR WideBuffer[BufferLen];
	UTF8CHAR Utf8Buffer[BufferLen];
	ANSICHAR AnsiBuffer[BufferLen];

	const WIDECHAR* WidePrefix = WIDETEXT("ABCD");
	const UTF8CHAR* Utf8Prefix = UTF8TEXT("ABCD");
	const ANSICHAR* AnsiPrefix = "ABCD";
	const WIDECHAR* WideTest = WIDETEXT("12345");
	const UTF8CHAR* Utf8Test = UTF8TEXT("12345");
	const ANSICHAR* AnsiTest = "12345";
	const WIDECHAR* WidePrefixPlusTest = WIDETEXT("ABCD12345");
	const UTF8CHAR* Utf8PrefixPlusTest = UTF8TEXT("ABCD12345");
	const ANSICHAR* AnsiPrefixPlusTest = "ABCD12345";
	constexpr int32 PrefixLen = 4;
	constexpr int32 TestLen = 5;
	constexpr int32 PrefixPlusTestLen = 9;

	auto Reset = [&WideBuffer, &Utf8Buffer, &AnsiBuffer, WidePrefix, Utf8Prefix, AnsiPrefix, BufferLen]()
		{
			int32 n = 0;
			for (; n < PrefixLen+1; ++n)
			{
				WideBuffer[n] = WidePrefix[n];
				Utf8Buffer[n] = Utf8Prefix[n];
				AnsiBuffer[n] = AnsiPrefix[n];
			}
			for (; n < BufferLen; ++n)
			{
				WideBuffer[n] = '%';
				Utf8Buffer[n] = UTF8CHAR('%'); 
				AnsiBuffer[n] = '%';
			}
		};

	Reset();
	FCStringWide::Strcat(WideBuffer, WideTest);
	FCStringUtf8::Strcat(Utf8Buffer, Utf8Test);
	FCStringAnsi::Strcat(AnsiBuffer, AnsiTest);
	this->TestTrue(TEXT("WideStrcat"),
		WideBuffer[PrefixPlusTestLen] == 0 && FCStringWide::Strcmp(WidePrefixPlusTest, WideBuffer) == 0 && WideBuffer[PrefixPlusTestLen + 1] == '%');
	this->TestTrue(TEXT("Utf8Strcat"),
		Utf8Buffer[PrefixPlusTestLen] == 0 && FCStringUtf8::Strcmp(Utf8PrefixPlusTest, Utf8Buffer) == 0 && Utf8Buffer[PrefixPlusTestLen + 1] == '%');
	this->TestTrue(TEXT("AnsiStrcat"),
		AnsiBuffer[PrefixPlusTestLen] == 0 && FCStringAnsi::Strcmp(AnsiPrefixPlusTest, AnsiBuffer) == 0 && AnsiBuffer[PrefixPlusTestLen + 1] == '%');

	Reset();
	FCStringWide::StrncatTruncateDest(WideBuffer, PrefixPlusTestLen + 10, WideTest);
	FCStringUtf8::StrncatTruncateDest(Utf8Buffer, PrefixPlusTestLen + 10, Utf8Test);
	FCStringAnsi::StrncatTruncateDest(AnsiBuffer, PrefixPlusTestLen + 10, AnsiTest);
	this->TestTrue(TEXT("WideStrncatTruncateDestTestLenPlus10"),
		WideBuffer[PrefixPlusTestLen] == 0 && FCStringWide::Strcmp(WidePrefixPlusTest, WideBuffer) == 0 && WideBuffer[PrefixPlusTestLen + 1] == '%');
	this->TestTrue(TEXT("Utf8StrncatTruncateDestTestLenPlus10"),
		Utf8Buffer[PrefixPlusTestLen] == 0 && FCStringUtf8::Strcmp(Utf8PrefixPlusTest, Utf8Buffer) == 0 && Utf8Buffer[PrefixPlusTestLen + 1] == '%');
	this->TestTrue(TEXT("AnsiStrncatTruncateDestTestLenPlus10"),
		AnsiBuffer[PrefixPlusTestLen] == 0 && FCStringAnsi::Strcmp(AnsiPrefixPlusTest, AnsiBuffer) == 0 && AnsiBuffer[PrefixPlusTestLen + 1] == '%');

	Reset();
	FCStringWide::StrncatTruncateDest(WideBuffer, PrefixPlusTestLen + 1, WideTest);
	FCStringUtf8::StrncatTruncateDest(Utf8Buffer, PrefixPlusTestLen + 1, Utf8Test);
	FCStringAnsi::StrncatTruncateDest(AnsiBuffer, PrefixPlusTestLen + 1, AnsiTest);
	this->TestTrue(TEXT("WideStrncatTruncateDestTestLenPlus1"),
		WideBuffer[PrefixPlusTestLen] == 0 && FCStringWide::Strcmp(WidePrefixPlusTest, WideBuffer) == 0 && WideBuffer[PrefixPlusTestLen + 1] == '%');
	this->TestTrue(TEXT("Utf8StrncatTruncateDestTestLenPlus1"),
		Utf8Buffer[PrefixPlusTestLen] == 0 && FCStringUtf8::Strcmp(Utf8PrefixPlusTest, Utf8Buffer) == 0 && Utf8Buffer[PrefixPlusTestLen + 1] == '%');
	this->TestTrue(TEXT("AnsiStrncatTruncateDestTestLenPlus1"),
		AnsiBuffer[PrefixPlusTestLen] == 0 && FCStringAnsi::Strcmp(AnsiPrefixPlusTest, AnsiBuffer) == 0 && AnsiBuffer[PrefixPlusTestLen + 1] == '%');

	Reset();
	FCStringWide::StrncatTruncateDest(WideBuffer, PrefixPlusTestLen, WideTest);
	FCStringUtf8::StrncatTruncateDest(Utf8Buffer, PrefixPlusTestLen, Utf8Test);
	FCStringAnsi::StrncatTruncateDest(AnsiBuffer, PrefixPlusTestLen, AnsiTest);
	this->TestTrue(TEXT("WideStrncatTruncateDestTestLen"),
		WideBuffer[PrefixPlusTestLen - 1] == 0 && WideBuffer[PrefixPlusTestLen] == '%' && FCStringWide::Strncmp(WidePrefixPlusTest, WideBuffer, PrefixPlusTestLen - 1) == 0 && WideBuffer[PrefixPlusTestLen + 1] == '%');
	this->TestTrue(TEXT("Utf8StrncatTruncateDestTestLen"),
		Utf8Buffer[PrefixPlusTestLen - 1] == 0 && Utf8Buffer[PrefixPlusTestLen] == '%' && FCStringUtf8::Strncmp(Utf8PrefixPlusTest, Utf8Buffer, PrefixPlusTestLen - 1) == 0 && Utf8Buffer[PrefixPlusTestLen + 1] == '%');
	this->TestTrue(TEXT("AnsiStrncatTruncateDestTestLen"),
		AnsiBuffer[PrefixPlusTestLen - 1] == 0 && AnsiBuffer[PrefixPlusTestLen] == '%' && FCStringAnsi::Strncmp(AnsiPrefixPlusTest, AnsiBuffer, PrefixPlusTestLen - 1) == 0 && AnsiBuffer[PrefixPlusTestLen + 1] == '%');

	Reset();
	FCStringWide::StrncatTruncateDest(WideBuffer, PrefixPlusTestLen - 1, WideTest);
	FCStringUtf8::StrncatTruncateDest(Utf8Buffer, PrefixPlusTestLen - 1, Utf8Test);
	FCStringAnsi::StrncatTruncateDest(AnsiBuffer, PrefixPlusTestLen - 1, AnsiTest);
	this->TestTrue(TEXT("WideStrncatTruncateDestTestLenMinus1"),
		WideBuffer[PrefixPlusTestLen - 2] == 0 && WideBuffer[PrefixPlusTestLen - 1] == '%' && FCStringWide::Strncmp(WidePrefixPlusTest, WideBuffer, PrefixPlusTestLen - 2) == 0 && WideBuffer[PrefixPlusTestLen + 1] == '%');
	this->TestTrue(TEXT("Utf8StrncatTruncateDestTestLenMinus1"),
		Utf8Buffer[PrefixPlusTestLen - 2] == 0 && Utf8Buffer[PrefixPlusTestLen - 1] == '%' && FCStringUtf8::Strncmp(Utf8PrefixPlusTest, Utf8Buffer, PrefixPlusTestLen - 2) == 0 && Utf8Buffer[PrefixPlusTestLen + 1] == '%');
	this->TestTrue(TEXT("AnsiStrncatTruncateDestTestLenMinus1"),
		AnsiBuffer[PrefixPlusTestLen - 2] == 0 && AnsiBuffer[PrefixPlusTestLen - 1] == '%' && FCStringAnsi::Strncmp(AnsiPrefixPlusTest, AnsiBuffer, PrefixPlusTestLen - 2) == 0 && AnsiBuffer[PrefixPlusTestLen + 1] == '%');

	Reset();
	FCStringWide::StrncatTruncateDest(WideBuffer, PrefixLen + 2, WideTest);
	FCStringUtf8::StrncatTruncateDest(Utf8Buffer, PrefixLen + 2, Utf8Test);
	FCStringAnsi::StrncatTruncateDest(AnsiBuffer, PrefixLen + 2, AnsiTest);
	this->TestTrue(TEXT("WideStrncatTruncateDestTwoLen"),
		WideBuffer[PrefixLen + 1] == 0 && WideBuffer[PrefixLen + 2] == '%' && FCStringWide::Strncmp(WidePrefixPlusTest, WideBuffer, PrefixLen + 1) == 0 && WideBuffer[PrefixPlusTestLen + 1] == '%');
	this->TestTrue(TEXT("Utf8StrncatTruncateDestTwoLen"),
		Utf8Buffer[PrefixLen + 1] == 0 && Utf8Buffer[PrefixLen + 2] == '%' && FCStringUtf8::Strncmp(Utf8PrefixPlusTest, Utf8Buffer, PrefixLen + 1) == 0 && Utf8Buffer[PrefixPlusTestLen + 1] == '%');
	this->TestTrue(TEXT("AnsiStrncatTruncateDestTwoLen"),
		AnsiBuffer[PrefixLen + 1] == 0 && AnsiBuffer[PrefixLen + 2] == '%' && FCStringAnsi::Strncmp(AnsiPrefixPlusTest, AnsiBuffer, PrefixLen + 1) == 0 && AnsiBuffer[PrefixPlusTestLen + 1] == '%');

	Reset();
	FCStringWide::StrncatTruncateDest(WideBuffer, PrefixLen + 1, WideTest);
	FCStringUtf8::StrncatTruncateDest(Utf8Buffer, PrefixLen + 1, Utf8Test);
	FCStringAnsi::StrncatTruncateDest(AnsiBuffer, PrefixLen + 1, AnsiTest);
	this->TestTrue(TEXT("WideStrncatTruncateDestOneLen"),
		WideBuffer[PrefixLen] == 0 && WideBuffer[PrefixLen + 1] == '%' && FCStringWide::Strncmp(WidePrefixPlusTest, WideBuffer, PrefixLen) == 0 && WideBuffer[PrefixPlusTestLen + 1] == '%');
	this->TestTrue(TEXT("Utf8StrncatTruncateDestOneLen"),
		Utf8Buffer[PrefixLen] == 0 && Utf8Buffer[PrefixLen + 1] == '%' && FCStringUtf8::Strncmp(Utf8PrefixPlusTest, Utf8Buffer, PrefixLen) == 0 && Utf8Buffer[PrefixPlusTestLen + 1] == '%');
	this->TestTrue(TEXT("AnsiStrncatTruncateDestOneLen"),
		AnsiBuffer[PrefixLen] == 0 && AnsiBuffer[PrefixLen + 1] == '%' && FCStringAnsi::Strncmp(AnsiPrefixPlusTest, AnsiBuffer, PrefixLen) == 0 && AnsiBuffer[PrefixPlusTestLen + 1] == '%');

	Reset();
	FCStringWide::StrncatTruncateDest(WideBuffer, PrefixLen, WideTest);
	FCStringUtf8::StrncatTruncateDest(Utf8Buffer, PrefixLen, Utf8Test);
	FCStringAnsi::StrncatTruncateDest(AnsiBuffer, PrefixLen, AnsiTest);
	this->TestTrue(TEXT("WideStrncatTruncateDestZeroLen"),
		WideBuffer[PrefixLen] == 0 && WideBuffer[PrefixLen + 1] == '%' && FCStringWide::Strncmp(WidePrefixPlusTest, WideBuffer, PrefixLen) == 0 && WideBuffer[PrefixPlusTestLen + 1] == '%');
	this->TestTrue(TEXT("Utf8StrncatTruncateDestZeroLen"),
		Utf8Buffer[PrefixLen] == 0 && Utf8Buffer[PrefixLen + 1] == '%' && FCStringUtf8::Strncmp(Utf8PrefixPlusTest, Utf8Buffer, PrefixLen) == 0 && Utf8Buffer[PrefixPlusTestLen + 1] == '%');
	this->TestTrue(TEXT("AnsiStrncatTruncateDestZeroLen"),
		AnsiBuffer[PrefixLen] == 0 && AnsiBuffer[PrefixLen + 1] == '%' && FCStringAnsi::Strncmp(AnsiPrefixPlusTest, AnsiBuffer, PrefixLen) == 0 && AnsiBuffer[PrefixPlusTestLen + 1] == '%');

	Reset();
	FCStringWide::StrncatTruncateSrc(WideBuffer, WideTest, TestLen + 10);
	FCStringUtf8::StrncatTruncateSrc(Utf8Buffer, Utf8Test, TestLen + 10);
	FCStringAnsi::StrncatTruncateSrc(AnsiBuffer, AnsiTest, TestLen + 10);
	this->TestTrue(TEXT("WideStrncatTruncateSrcTestLenPlus10"),
		WideBuffer[PrefixPlusTestLen] == 0 && FCStringWide::Strcmp(WidePrefixPlusTest, WideBuffer) == 0 && WideBuffer[PrefixPlusTestLen + 1] == '%');
	this->TestTrue(TEXT("Utf8StrncatTruncateSrcTestLenPlus10"),
		Utf8Buffer[PrefixPlusTestLen] == 0 && FCStringUtf8::Strcmp(Utf8PrefixPlusTest, Utf8Buffer) == 0 && Utf8Buffer[PrefixPlusTestLen + 1] == '%');
	this->TestTrue(TEXT("AnsiStrncatTruncateSrcTestLenPlus10"),
		AnsiBuffer[PrefixPlusTestLen] == 0 && FCStringAnsi::Strcmp(AnsiPrefixPlusTest, AnsiBuffer) == 0 && AnsiBuffer[PrefixPlusTestLen + 1] == '%');

	Reset();
	FCStringWide::StrncatTruncateSrc(WideBuffer, WideTest, TestLen);
	FCStringUtf8::StrncatTruncateSrc(Utf8Buffer, Utf8Test, TestLen);
	FCStringAnsi::StrncatTruncateSrc(AnsiBuffer, AnsiTest, TestLen);
	this->TestTrue(TEXT("WideStrncatTruncateSrcTestLenPlus1"),
		WideBuffer[PrefixPlusTestLen] == 0 && FCStringWide::Strcmp(WidePrefixPlusTest, WideBuffer) == 0 && WideBuffer[PrefixPlusTestLen + 1] == '%');
	this->TestTrue(TEXT("Utf8StrncatTruncateSrcTestLenPlus1"),
		Utf8Buffer[PrefixPlusTestLen] == 0 && FCStringUtf8::Strcmp(Utf8PrefixPlusTest, Utf8Buffer) == 0 && Utf8Buffer[PrefixPlusTestLen + 1] == '%');
	this->TestTrue(TEXT("AnsiStrncatTruncateSrcTestLenPlus1"),
		AnsiBuffer[PrefixPlusTestLen] == 0 && FCStringAnsi::Strcmp(AnsiPrefixPlusTest, AnsiBuffer) == 0 && AnsiBuffer[PrefixPlusTestLen + 1] == '%');

	Reset();
	FCStringWide::StrncatTruncateSrc(WideBuffer, WideTest, TestLen - 1);
	FCStringUtf8::StrncatTruncateSrc(Utf8Buffer, Utf8Test, TestLen - 1);
	FCStringAnsi::StrncatTruncateSrc(AnsiBuffer, AnsiTest, TestLen - 1);
	this->TestTrue(TEXT("WideStrncatTruncateSrcTestLenMinus1"),
		WideBuffer[PrefixPlusTestLen - 1] == 0 && WideBuffer[PrefixPlusTestLen] == '%' && FCStringWide::Strncmp(WidePrefixPlusTest, WideBuffer, PrefixPlusTestLen - 1) == 0 && WideBuffer[PrefixPlusTestLen + 1] == '%');
	this->TestTrue(TEXT("Utf8StrncatTruncateSrcTestLenMinus1"),
		Utf8Buffer[PrefixPlusTestLen - 1] == 0 && Utf8Buffer[PrefixPlusTestLen] == '%' && FCStringUtf8::Strncmp(Utf8PrefixPlusTest, Utf8Buffer, PrefixPlusTestLen - 1) == 0 && Utf8Buffer[PrefixPlusTestLen + 1] == '%');
	this->TestTrue(TEXT("AnsiStrncatTruncateSrcTestLenMinus1"),
		AnsiBuffer[PrefixPlusTestLen - 1] == 0 && AnsiBuffer[PrefixPlusTestLen] == '%' && FCStringAnsi::Strncmp(AnsiPrefixPlusTest, AnsiBuffer, PrefixPlusTestLen - 1) == 0 && AnsiBuffer[PrefixPlusTestLen + 1] == '%');

	Reset();
	FCStringWide::StrncatTruncateSrc(WideBuffer, WideTest, TestLen - 2);
	FCStringUtf8::StrncatTruncateSrc(Utf8Buffer, Utf8Test, TestLen - 2);
	FCStringAnsi::StrncatTruncateSrc(AnsiBuffer, AnsiTest, TestLen - 2);
	this->TestTrue(TEXT("WideStrncatTruncateSrcTestLenMinus2"),
		WideBuffer[PrefixPlusTestLen - 2] == 0 && WideBuffer[PrefixPlusTestLen - 1] == '%' && FCStringWide::Strncmp(WidePrefixPlusTest, WideBuffer, PrefixPlusTestLen - 2) == 0 && WideBuffer[PrefixPlusTestLen + 1] == '%');
	this->TestTrue(TEXT("Utf8StrncatTruncateSrcTestLenMinus2"),
		Utf8Buffer[PrefixPlusTestLen - 2] == 0 && Utf8Buffer[PrefixPlusTestLen - 1] == '%' && FCStringUtf8::Strncmp(Utf8PrefixPlusTest, Utf8Buffer, PrefixPlusTestLen - 2) == 0 && Utf8Buffer[PrefixPlusTestLen + 1] == '%');
	this->TestTrue(TEXT("AnsiStrncatTruncateSrcTestLenMinus2"),
		AnsiBuffer[PrefixPlusTestLen - 2] == 0 && AnsiBuffer[PrefixPlusTestLen - 1] == '%' && FCStringAnsi::Strncmp(AnsiPrefixPlusTest, AnsiBuffer, PrefixPlusTestLen - 2) == 0 && AnsiBuffer[PrefixPlusTestLen + 1] == '%');

	Reset();
	FCStringWide::StrncatTruncateSrc(WideBuffer, WideTest, 1);
	FCStringUtf8::StrncatTruncateSrc(Utf8Buffer, Utf8Test, 1);
	FCStringAnsi::StrncatTruncateSrc(AnsiBuffer, AnsiTest, 1);
	this->TestTrue(TEXT("WideStrncatTruncateSrcOneLen"),
		WideBuffer[PrefixLen + 1] == 0 && WideBuffer[PrefixLen + 2] == '%' && FCStringWide::Strncmp(WidePrefixPlusTest, WideBuffer, PrefixLen + 1) == 0 && WideBuffer[PrefixPlusTestLen + 1] == '%');
	this->TestTrue(TEXT("Utf8StrncatTruncateSrcOneLen"),
		Utf8Buffer[PrefixLen + 1] == 0 && Utf8Buffer[PrefixLen + 2] == '%' && FCStringUtf8::Strncmp(Utf8PrefixPlusTest, Utf8Buffer, PrefixLen + 1) == 0 && Utf8Buffer[PrefixPlusTestLen + 1] == '%');
	this->TestTrue(TEXT("AnsiStrncatTruncateSrcOneLen"),
		AnsiBuffer[PrefixLen + 1] == 0 && AnsiBuffer[PrefixLen + 2] == '%' && FCStringAnsi::Strncmp(AnsiPrefixPlusTest, AnsiBuffer, PrefixLen + 1) == 0 && AnsiBuffer[PrefixPlusTestLen + 1] == '%');

	Reset();
	FCStringWide::StrncatTruncateSrc(WideBuffer, WideTest, 0);
	FCStringUtf8::StrncatTruncateSrc(Utf8Buffer, Utf8Test, 0);
	FCStringAnsi::StrncatTruncateSrc(AnsiBuffer, AnsiTest, 0);
	this->TestTrue(TEXT("WideStrncatTruncateSrcZeroLen"),
		WideBuffer[PrefixLen] == 0 && WideBuffer[PrefixLen + 1] == '%' && FCStringWide::Strncmp(WidePrefixPlusTest, WideBuffer, PrefixLen) == 0 && WideBuffer[PrefixPlusTestLen + 1] == '%');
	this->TestTrue(TEXT("Utf8StrncatTruncateSrcZeroLen"),
		Utf8Buffer[PrefixLen] == 0 && Utf8Buffer[PrefixLen + 1] == '%' && FCStringUtf8::Strncmp(Utf8PrefixPlusTest, Utf8Buffer, PrefixLen) == 0 && Utf8Buffer[PrefixPlusTestLen + 1] == '%');
	this->TestTrue(TEXT("AnsiStrncatTruncateSrcZeroLen"),
		AnsiBuffer[PrefixLen] == 0 && AnsiBuffer[PrefixLen + 1] == '%' && FCStringAnsi::Strncmp(AnsiPrefixPlusTest, AnsiBuffer, PrefixLen) == 0 && AnsiBuffer[PrefixPlusTestLen + 1] == '%');

	return true;
}

#endif // WITH_DEV_AUTOMATION_TESTS
