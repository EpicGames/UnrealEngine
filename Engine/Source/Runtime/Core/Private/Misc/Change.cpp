// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/Change.h"

#include "Containers/UnrealString.h"
#include "Misc/FeedbackContext.h"
#include "Templates/EnableIf.h"

class UObject;

void FChange::PrintToLog( FFeedbackContext& FeedbackContext, const int32 IndentLevel )
{
	FString IndentString;
	for( int32 IndentIter = 0; IndentIter < IndentLevel; ++IndentIter )
	{
		IndentString += '\t';
	}

	FeedbackContext.Log( IndentString + ToString() );
}


TUniquePtr<FChange> FCompoundChange::Execute( UObject* Object )
{
	FCompoundChangeInput RevertInput;

	RevertInput.Subchanges.Reserve( Input.Subchanges.Num() );

	// Iterate backwards, so the changes will be executed in the reverse order they were added in.
	for( int32 ChangeIndex = Input.Subchanges.Num() - 1; ChangeIndex >= 0; --ChangeIndex )
	{
		TUniquePtr<FChange>& Subchange = Input.Subchanges[ ChangeIndex ];

		// Skip null changes.  We allow null entries because it can be convienent when building up
		// a compound change with a lot of subchanges that might only be no-ops.
		if( Subchange != nullptr )
		{
			RevertInput.Subchanges.Add( Subchange->Execute( Object ) );
		}
	}

	return ( RevertInput.Subchanges.Num() > 0 ) ? MakeUnique<FCompoundChange>( MoveTemp( RevertInput ) ) : nullptr;
}


FString FCompoundChange::ToString() const
{
	FString Text( TEXT( "Compound Change" ) );

	if( Input.Subchanges.Num() > 0 )
	{
		int32 TotalValidSubchanges = 0;
		for( const TUniquePtr<FChange>& Subchange : Input.Subchanges )
		{
			if( Subchange != nullptr )
			{
				++TotalValidSubchanges;
			}
		}

		Text += FString::Printf( TEXT( " (%i sub-change%s):" ), TotalValidSubchanges, TotalValidSubchanges == 1 ? TEXT( "s" ) : TEXT( "" ) );

		for (int32 ChangeIndex = Input.Subchanges.Num() - 1; ChangeIndex >= 0; --ChangeIndex)
		{
			const TUniquePtr<FChange>& Subchange = Input.Subchanges[ChangeIndex];
			if (Subchange != nullptr)
			{
				Text += TEXT("\n\t");
				Text += Subchange->ToString();
			}
		}
	}
	else
	{
		Text += TEXT( " (empty)" );
	}

	return Text;
}


void FCompoundChange::PrintToLog( FFeedbackContext& FeedbackContext, const int32 IndentLevel )
{
	const bool bWantCompoundHeadersAndIndentation = false;	// NOTE: This can be useful to set to 'true' if you need to see the actual hierarchy

	if( bWantCompoundHeadersAndIndentation )
	{
		// Call parent implementation to print our own change header, first
		FChange::PrintToLog( FeedbackContext, IndentLevel );
	}

	// Print all of our sub-changes, too!  This will recursively indent all nested compound changes.
	// Iterate backwards (changes will be executed in the reverse order they were added in.)
	for( int32 ChangeIndex = Input.Subchanges.Num() - 1; ChangeIndex >= 0; --ChangeIndex )
	{
		TUniquePtr<FChange>& Subchange = Input.Subchanges[ ChangeIndex ];

		if( Subchange != nullptr )
		{
			Subchange->PrintToLog( FeedbackContext, bWantCompoundHeadersAndIndentation ? ( IndentLevel + 1 ) : IndentLevel );
		}
	}
}
