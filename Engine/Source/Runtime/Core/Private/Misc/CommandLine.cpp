// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/CommandLine.h"
#include "Misc/MessageDialog.h"
#include "Logging/LogMacros.h"
#include "Misc/Parse.h"
#include "Misc/CoreMisc.h"
#include "Internationalization/Text.h"
#include "Internationalization/Internationalization.h"

/*-----------------------------------------------------------------------------
	FCommandLine
-----------------------------------------------------------------------------*/

bool FCommandLine::bIsInitialized = false;
uint32 FCommandLine::CmdLineVersion = 0;
TCHAR FCommandLine::CmdLine[FCommandLine::MaxCommandLineSize] = {};
TCHAR FCommandLine::OriginalCmdLine[FCommandLine::MaxCommandLineSize] = {};
TCHAR FCommandLine::LoggingCmdLine[FCommandLine::MaxCommandLineSize] = {};
TCHAR FCommandLine::LoggingOriginalCmdLine[FCommandLine::MaxCommandLineSize] = {};
TMap<FString, FCommandLine::FRegisteredArgData> FCommandLine::RegisteredArgs;

class FInitializedCommandlinesArray : public TArray<FString>
{
public:
	FInitializedCommandlinesArray()
	{
		SetNum((int)ECommandLineArgumentFlags::AllContexts);
		this->operator[]((int)ECommandLineArgumentFlags::AllContexts - 1) = TEXT(" -Multiprocess");
	}
};

FString& FCommandLine::GetSubprocessCommandLine_Internal(ECommandLineArgumentFlags ContextFlags)
{
	static FInitializedCommandlinesArray SubprocessCommandLines;
	int Index = (int)(ContextFlags & ECommandLineArgumentFlags::AllContexts) - 1;
	check(Index >= 0);
	return SubprocessCommandLines[Index];
}

bool FCommandLine::IsInitialized()
{
	return bIsInitialized;
}

uint32 FCommandLine::GetCommandLineVersion()
{
	return CmdLineVersion;
}

void FCommandLine::Reset()
{
	CmdLine[0] = TEXT('\0');
	OriginalCmdLine[0] = TEXT('\0');
	LoggingCmdLine[0] = TEXT('\0');
	LoggingOriginalCmdLine[0] = TEXT('\0');
	bIsInitialized = false;
	CmdLineVersion++;
}

const TCHAR* FCommandLine::Get()
{
	UE_CLOGF(!bIsInitialized, LogInit, Fatal, "Attempting to get the command line but it hasn't been initialized yet.");
	return CmdLine;
}

const TCHAR* FCommandLine::GetForLogging()
{
	UE_CLOGF(!bIsInitialized, LogInit, Fatal, "Attempting to get the command line but it hasn't been initialized yet.");
	return LoggingCmdLine;
}

const TCHAR* FCommandLine::GetOriginal()
{
	UE_CLOGF(!bIsInitialized, LogInit, Fatal, "Attempting to get the command line but it hasn't been initialized yet.");
	return OriginalCmdLine;
}

const TCHAR* FCommandLine::GetOriginalForLogging()
{
	UE_CLOGF(!bIsInitialized, LogInit, Fatal, "Attempting to get the command line but it hasn't been initialized yet.");
	return LoggingOriginalCmdLine;
}

bool FCommandLine::Set(const TCHAR* NewCommandLine)
{
	if (!bIsInitialized)
	{
		FCString::Strncpy(OriginalCmdLine, NewCommandLine, UE_ARRAY_COUNT(OriginalCmdLine));
		FCString::Strncpy(LoggingOriginalCmdLine, NewCommandLine, UE_ARRAY_COUNT(LoggingOriginalCmdLine));
	}

	FCString::Strncpy( CmdLine, NewCommandLine, UE_ARRAY_COUNT(CmdLine) );
	FCString::Strncpy(LoggingCmdLine, NewCommandLine, UE_ARRAY_COUNT(LoggingCmdLine));

	// If configured as part of the build, strip out any unapproved args
	ApplyCommandLineAllowList();

	bIsInitialized = true;
	CmdLineVersion++;

	// Check that the command line doesn't exceed our internal storage limit
	int NewCommandLineLength = FCString::Strlen(NewCommandLine);
	if (NewCommandLineLength >= UE_ARRAY_COUNT(CmdLine))
	{
		FText ErrorMessage = FText::Format(NSLOCTEXT("Engine", "CmdLineTooLong", "Error: Command-line exceeds internal storage limit.\nLength = {0}\nLimit={1}"),
			NewCommandLineLength, int32(UE_ARRAY_COUNT(CmdLine)-1));
#if !UE_BUILD_SHIPPING
		// We can't call FApp::IsUnattended, because command line setting can be called before FApp::IsUnattended can read from the command line
		if (!FParse::Param(FCommandLine::Get(), TEXT("UNATTENDED")))
		{
			FMessageDialog::Open(EAppMsgType::Ok, ErrorMessage);
			return false;
		}
#endif
		UE_LOGF(LogInit, Fatal, "%ls", *ErrorMessage.ToString());
	}

	// Check for the '-' that normal ones get converted to in Outlook. It's important to do it AFTER the command line is initialized
	if (StringHasBadDashes(NewCommandLine))
	{
		FText ErrorMessage = FText::Format(NSLOCTEXT("Engine", "CmdLineHasInvalidChar", "Error: Command-line contains an invalid '-' character, likely pasted from an email.\nCmdline = {0}"),
			FText::FromString(NewCommandLine));
#if !UE_BUILD_SHIPPING
		// We can't call FApp::IsUnattended, because command line setting can be called before FApp::IsUnattended can read from the command line
		if (!FParse::Param(FCommandLine::Get(), TEXT("UNATTENDED")))
		{
			FMessageDialog::Open(EAppMsgType::Ok, ErrorMessage);
			return false;
		}
#endif
		UE_LOGF(LogInit, Fatal, "%ls", *ErrorMessage.ToString());
	}

	return true;
}

void FCommandLine::Append(const TCHAR* AppendString)
{
	FCString::StrncatTruncateDest( CmdLine, UE_ARRAY_COUNT(CmdLine), AppendString );
	CmdLineVersion++;
	// If configured as part of the build, strip out any unapproved args
	ApplyCommandLineAllowList();
}

bool FCommandLine::IsCommandLineLoggingFiltered()
{
#ifdef FILTER_COMMANDLINE_LOGGING
	return true;
#else
	return false;
#endif
}

bool FCommandLine::FilterCLIUsingGrammarBasedParser(TCHAR* OutLine, int32 MaxLen, const TCHAR* InLine, const TArrayView<FString>& AllowedList)
{
	if (MaxLen == 0)
	{
		return false;
	}
	check(OutLine && MaxLen > 0);

	// If nothing is allowed, then the output is an empty string
	if (AllowedList.Num() == 0)
	{
		*OutLine = TCHAR(0);
		return true;
	}

	TCHAR* Write = OutLine;
	bool bOutOfSpace = false;

	auto OnCmd = [OutLine, MaxLen, &bOutOfSpace , &Write, &AllowedList] (FStringView Key, FStringView Value) mutable
	{
		// Filter
		// Trim the first leading '-'
		// This brings the behaviour inline with FCommandLine::Parse
		FStringView ToTest = Key.StartsWith(TEXT("-")) ? Key.RightChop(1) : Key;

		if (!AllowedList.Contains(ToTest))
		{
			return;
		}

		// Destination Accounting
		int32 WriteLength = Key.Len() + Value.Len() + (Write != OutLine);
		bOutOfSpace |= (WriteLength >= MaxLen);
		if (bOutOfSpace)
		{
			return;
		}
		MaxLen -= WriteLength;

		// Append
		if (Write != OutLine)
		{
			*Write++ = TCHAR(' ');
		}

		FMemory::Memmove(Write, Key.GetData(), sizeof(TCHAR) * Key.Len());
		Write += Key.Len();
		if (!Value.IsEmpty())
		{
			*Write++ = TCHAR('=');
			FMemory::Memmove(Write, Value.GetData(), sizeof(TCHAR) * Value.Len());
			Write += Value.Len();
		}
	};
	
	FParse::GrammarBasedCLIParse(InLine, OnCmd, FParse::EGrammarBasedParseFlags::AllowQuotedCommands);

	if (bOutOfSpace)
	{
		return false;
	}

	*Write = TCHAR(0);
	return true;
}

#if UE_COMMAND_LINE_USES_ALLOW_LIST
TArray<FString> FCommandLine::ApprovedArgs;
TArray<FString> FCommandLine::FilterArgsForLogging;

#ifndef UE_OVERRIDE_COMMAND_LINE_ALLOW_LIST
	#ifdef OVERRIDE_COMMANDLINE_WHITELIST
		#pragma message("Use UE_OVERRIDE_COMMAND_LINE_ALLOW_LIST instead")
		#define UE_OVERRIDE_COMMAND_LINE_ALLOW_LIST OVERRIDE_COMMANDLINE_WHITELIST
	#endif
#endif

#ifdef UE_OVERRIDE_COMMAND_LINE_ALLOW_LIST
/**
 * When overriding this setting make sure that your define looks like the following in your .cs file:
 *
 *		GlobalDefinitions.Add("UE_OVERRIDE_COMMAND_LINE_ALLOW_LIST=\"-arg1 -arg2 -arg3 -arg4\"");
 *
 * The important part is the \" as they quotes get stripped off by the compiler without them
 */
const TCHAR* OverrideList = TEXT(UE_OVERRIDE_COMMAND_LINE_ALLOW_LIST);
#else
// Default list most conservative restrictions
const TCHAR* OverrideList = TEXT("-fullscreen /windowed");
#endif

#ifdef FILTER_COMMANDLINE_LOGGING
/**
 * When overriding this setting make sure that your define looks like the following in your .cs file:
 *
 *		GlobalDefinitions.Add("FILTER_COMMANDLINE_LOGGING=\"-arg1 -arg2 -arg3 -arg4\"");
 *
 * The important part is the \" as they quotes get stripped off by the compiler without them
 */
const TCHAR* FilterForLoggingList = TEXT(FILTER_COMMANDLINE_LOGGING);
#else
const TCHAR* FilterForLoggingList = TEXT("");
#endif

void FCommandLine::ApplyCommandLineAllowList()
{
	if (ApprovedArgs.Num() == 0)
	{
		TArray<FString> Ignored;
		FCommandLine::Parse(OverrideList, ApprovedArgs, Ignored);
	}
	if (FilterArgsForLogging.Num() == 0)
	{
		TArray<FString> Ignored;
		FCommandLine::Parse(FilterForLoggingList, FilterArgsForLogging, Ignored);
	}
	// Process the original command line
	FilterCLIUsingGrammarBasedParser(OriginalCmdLine, UE_ARRAY_COUNT(OriginalCmdLine), OriginalCmdLine, ApprovedArgs);
	// Process the current command line
	FilterCLIUsingGrammarBasedParser(CmdLine, UE_ARRAY_COUNT(CmdLine), CmdLine, ApprovedArgs);
	// Process the command line for logging purposes
	FilterCLIUsingGrammarBasedParser(LoggingCmdLine, UE_ARRAY_COUNT(LoggingCmdLine), LoggingCmdLine, FilterArgsForLogging);
	// Process the original command line for logging purposes
	FilterCLIUsingGrammarBasedParser(LoggingOriginalCmdLine, UE_ARRAY_COUNT(LoggingOriginalCmdLine), LoggingOriginalCmdLine, FilterArgsForLogging);
}

TArray<FString> FCommandLine::FilterCommandLine(TCHAR* CommandLine)
{
	TArray<FString> Ignored;
	TArray<FString> ParsedList;
	// Parse the command line list
	FCommandLine::Parse(CommandLine, ParsedList, Ignored);
	// Remove any that are not in our approved list
	for (int32 Index = 0; Index < ParsedList.Num(); Index++)
	{
		bool bFound = false;
		for (auto ApprovedArg : ApprovedArgs)
		{
			if (ParsedList[Index].StartsWith(ApprovedArg))
			{
				bFound = true;
				break;
			}
		}
		if (!bFound)
		{
			ParsedList.RemoveAt(Index);
			Index--;
		}
	}
	return ParsedList;
}

TArray<FString> FCommandLine::FilterCommandLineForLogging(TCHAR* CommandLine)
{
	TArray<FString> Ignored;
	TArray<FString> ParsedList;
	// Parse the command line list
	FCommandLine::Parse(CommandLine, ParsedList, Ignored);
	// Remove any that are not in our approved list
	for (int32 Index = 0; Index < ParsedList.Num(); Index++)
	{
		for (auto Filter : FilterArgsForLogging)
		{
			if (ParsedList[Index].StartsWith(Filter))
			{
				ParsedList.RemoveAt(Index);
				Index--;
				break;
			}
		}
	}
	return ParsedList;
}

void FCommandLine::BuildCommandLineAllowList(TCHAR* CommandLine, uint32 ArrayCount, const TArray<FString>& FilteredArgs)
{
	check(ArrayCount > 0);
	// Zero the whole string
	FMemory::Memzero(CommandLine, sizeof(TCHAR) * ArrayCount);

	uint32 StartIndex = 0;
	for (auto Arg : FilteredArgs)
	{
		if ((StartIndex + Arg.Len() + 2) < ArrayCount)
		{
			if (StartIndex != 0)
			{
				CommandLine[StartIndex++] = TEXT(' ');
			}
			CommandLine[StartIndex++] = TEXT('-');
			FCString::Strncpy(&CommandLine[StartIndex], *Arg, ArrayCount - StartIndex);
			StartIndex += Arg.Len();
		}
	}
}
#endif

void FCommandLine::RegisterArgument(FStringView Name, ECommandLineArgumentFlags Flags, FStringView Description)
{
	if ((Flags & ECommandLineArgumentFlags::AllContexts) == ECommandLineArgumentFlags::None)
	{
		return;
	}

	FRegisteredArgData& RegisteredArg = RegisteredArgs.FindOrAdd(FString(Name));
	RegisteredArg.Flags |= Flags;
	RegisteredArg.Description = Description;
}

void FCommandLine::AddToSubprocessCommandLine_Internal(const TCHAR* Param, ECommandLineArgumentFlags ApplicationContextFlags)
{
	FString& SubprocessCommandLine = GetSubprocessCommandLine_Internal(ApplicationContextFlags);

	check( Param != NULL );
	if (Param[0] != ' ')
	{
		SubprocessCommandLine += TEXT(" ");
	}
	SubprocessCommandLine += Param;
}

void FCommandLine::AddToSubprocessCommandLine( const TCHAR* Param, ECommandLineArgumentFlags ApplicationContextFlags )
{
	if ((ApplicationContextFlags & ECommandLineArgumentFlags::AllContexts) == ECommandLineArgumentFlags::None)
	{
		return;
	}

	AddToSubprocessCommandLine_Internal(Param, ApplicationContextFlags);
}

void FCommandLine::BuildSubprocessCommandLine(ECommandLineArgumentFlags ApplicationContextFlags, bool bOnlyInherited, FStringBuilderBase& OutCommandline)
{
	if ((ApplicationContextFlags & ECommandLineArgumentFlags::AllContexts) == ECommandLineArgumentFlags::None)
	{
		return;
	}

	if (!bOnlyInherited)
	{
		// Append all explicitly supplied subprocess arguments
		for (int ContextPermutation = 1; ContextPermutation < (int)ECommandLineArgumentFlags::AllContexts; ++ContextPermutation)
		{
			if (ContextPermutation & (int)ApplicationContextFlags)
			{
				OutCommandline.Append(GetSubprocessCommandLine_Internal((ECommandLineArgumentFlags)ContextPermutation));
			}
		}
		OutCommandline.Append(GetSubprocessCommandLine_Internal(ECommandLineArgumentFlags::AllContexts));
	}

	// Append all inherited arguments
	const TCHAR* Stream = Get();
	FString NextToken;
	while (FParse::Token(Stream, NextToken, false))
	{
		if (NextToken.StartsWith(TEXT("-")))
		{
			FStringView NextTokenView(NextToken);
			NextTokenView.RightChopInline(1); //Chop the leading dash from the view
			for (const TPair<FString, FRegisteredArgData>& RegisteredArg : RegisteredArgs)
			{
				if ((RegisteredArg.Value.Flags & ECommandLineArgumentFlags::Inherit) != ECommandLineArgumentFlags::Inherit)
				{
					continue;
				}

				if (((RegisteredArg.Value.Flags & ApplicationContextFlags) != ECommandLineArgumentFlags::None) && NextTokenView.StartsWith(RegisteredArg.Key))
				{
					if ((NextTokenView.Len() == RegisteredArg.Key.Len()) || (NextTokenView[RegisteredArg.Key.Len()] == TCHAR('=')))
					{
						if (OutCommandline.Len() != 0)
						{
							OutCommandline.AppendChar(TCHAR(' '));
						}
						OutCommandline.Append(NextToken);
						break;
					}
				}
			}
		}
	}
}

/** 
 * Removes the executable name from a commandline, denoted by parentheses.
 */
const TCHAR* FCommandLine::RemoveExeName(const TCHAR* InCmdLine)
{
	// Skip over executable that is in the commandline
	if( *InCmdLine=='\"' )
	{
		InCmdLine++;
		while( *InCmdLine && *InCmdLine!='\"' )
		{
			InCmdLine++;
		}
		if( *InCmdLine )
		{
			InCmdLine++;
		}
	}
	while( *InCmdLine && *InCmdLine!=' ' )
	{
		InCmdLine++;
	}
	// skip over any spaces at the start
	while (*InCmdLine == ' ')
	{
		InCmdLine++;
	}
	return InCmdLine;
}


/**
 * Parses a string into tokens, separating switches (beginning with -) from
 * other parameters
 *
 * @param	CmdLine		the string to parse
 * @param	Tokens		[out] filled with all parameters found in the string
 * @param	Switches	[out] filled with all switches found in the string
 */
void FCommandLine::Parse(const TCHAR* InCmdLine, TArray<FString>& Tokens, TArray<FString>& Switches)
{
	FString NextToken;
	while (FParse::Token(InCmdLine, NextToken, false))
	{
		if ((**NextToken == TCHAR('-')))
		{
			Switches.Add(NextToken.Mid(1));
			Tokens.Add(NextToken.Right(NextToken.Len() - 1));
		}
		else
		{
			Tokens.Add(MoveTemp(NextToken));
		}
	}
}

template<typename CharType>
FString BuildFromArgVImpl(const CharType* Prefix, int32 ArgC, CharType* ArgV[], const CharType* Suffix)
{
	FString Result;

	// loop over the parameters, skipping the first one (which is the executable name)
	for (int32 Arg = 1; Arg < ArgC; Arg++)
	{
		FString ThisArg = ArgV[Arg];
		if (ThisArg.Contains(TEXT(" ")) && !ThisArg.Contains(TEXT("\"")))
		{
			int32 EqualsAt = ThisArg.Find(TEXT("="));
			if (EqualsAt > 0 && ThisArg.Find(TEXT(" ")) > EqualsAt)
			{
				ThisArg = ThisArg.Left(EqualsAt + 1) + FString("\"") + ThisArg.RightChop(EqualsAt + 1) + FString("\"");

			}
			else
			{
				ThisArg = FString("\"") + ThisArg + FString("\"");
			}
		}

		Result += ThisArg;
		// put a space between each argument (not needed after the end)
		if (Arg + 1 < ArgC)
		{
			Result += TEXT(" ");
		}
	}

	// add the prefix and suffix if provided
	if (Prefix)
	{
		Result = FString::Printf(TEXT("%s %s"), StringCast<TCHAR>(Prefix).Get(), *Result);
	}

	if (Suffix)
	{
		Result = FString::Printf(TEXT("%s %s"), *Result, StringCast<TCHAR>(Suffix).Get());
	}

	return Result;
}

FString FCommandLine::BuildFromArgV(const WIDECHAR* Prefix, int32 ArgC, WIDECHAR* ArgV[], const WIDECHAR* Suffix)
{
	return BuildFromArgVImpl(Prefix, ArgC, ArgV, Suffix);
}

FString FCommandLine::BuildFromArgV(const ANSICHAR* Prefix, int32 ArgC, ANSICHAR* ArgV[], const ANSICHAR* Suffix)
{
	return BuildFromArgVImpl(Prefix, ArgC, ArgV, Suffix);
}
