// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/ComparisonUtility.h"

#include "Containers/StringView.h"
#include "CoreTypes.h"
#include "Misc/CString.h"

namespace UE::ComparisonUtility {

int32 CompareWithNumericSuffix(FName A, FName B)
{
	return CompareWithNumericSuffix(FNameBuilder(A).ToView(), FNameBuilder(B).ToView());
}

int32 CompareWithNumericSuffix(FStringView A, FStringView B)
{
	auto SplitNumericSuffix = [](FStringView Full, FStringView& OutPrefix, int64& OutSuffix)
	{
		int32 NumDigits = 0;
		OutSuffix = 0;
		if (!Full.IsEmpty())
		{
			int32 Magnitude = 1;
			const TCHAR* FullStart = Full.GetData();
			for (const TCHAR* C = FullStart + Full.Len() - 1; C >= FullStart && FChar::IsDigit(*C); --C)
			{
				OutSuffix += FChar::ConvertCharDigitToInt(*C) * Magnitude;
				Magnitude *= 10;
				++NumDigits;
			}
		}

		static constexpr int32 MaxDigitsInt64 = 19;
		if (NumDigits > MaxDigitsInt64)
		{
			// Too large to process as a number
			NumDigits = 0;
		}

		if (NumDigits > 0)
		{
			OutPrefix = Full.LeftChop(NumDigits);
			// OutSuffix has already been set by the parsing loop above
		}
		else
		{
			OutPrefix = Full;
			OutSuffix = INDEX_NONE;
		}
	};

	FStringView APrefix;
	int64 ASuffix = 0;
	SplitNumericSuffix(A, APrefix, ASuffix);

	FStringView BPrefix;
	int64 BSuffix = 0;
	SplitNumericSuffix(B, BPrefix, BSuffix);

	// Are the prefixes identical?
	const int32 PrefixResult = APrefix.Compare(BPrefix, ESearchCase::IgnoreCase);
	if (PrefixResult != 0)
	{
		return PrefixResult;
	}

	// If so, compare the suffixes too
	if (ASuffix < BSuffix)
	{
		return -1;
	}
	if (ASuffix > BSuffix)
	{
		return 1;
	}
	return 0;
}

int32 CompareNaturalOrder(FStringView A, FStringView B)
{
	auto BothAscii = [](TCHAR C1, TCHAR C2) -> bool
	{
		return ((static_cast<uint32>(C1) | static_cast<uint32>(C2)) & 0xffffff80) == 0;
	};

	auto ConsumeNumber = [](const TCHAR*& Str) -> int64
	{
		bool bOverflow = false;
		uint64 Number = 0;
		TCHAR C = *Str;
		do
		{
			Number *= 10;
			Number += FChar::ConvertCharDigitToInt(C);

			bOverflow = bOverflow || Number > static_cast<uint64>(MAX_int64);
			Str++;
			C = *Str;
		} while (FChar::IsDigit(C));

		if (bOverflow)
		{
			// Too large to process as a number
			Number = INDEX_NONE;
		}

		return Number;
	};
	
	const TCHAR* StringA = A.GetData();
	const TCHAR* StringB = B.GetData();
 
	while (true)
	{
		TCHAR CharA = *StringA;
		TCHAR CharB = *StringB;
 
		// Ignore underscores
		if (FChar::IsUnderscore(CharA))
		{
			StringA++;
			continue;
		}
 
		// Ignore underscores
		if (FChar::IsUnderscore(CharB))
		{
			StringB++;
			continue;
		}
 
		// Sort numerically when numbers are found 
		if (FChar::IsDigit(CharA) && FChar::IsDigit(CharB))
		{
			const int64 IntA = ConsumeNumber(StringA);
			const int64 IntB = ConsumeNumber(StringB);
 
			if (IntA != IntB)
			{
				return IntA < IntB ? -1 : 1;
			}
 
			continue;
		}
		else if (CharA == CharB)
		{
			// Reached the end of the string
			if (CharA == TCHAR('\0'))
			{
				// Strings compared equal, return shortest first
				return A.Len() == B.Len() ? 0 : A.Len() < B.Len() ? -1 : 1;
			}
		}
		else if (BothAscii(CharA, CharB))
		{
			if (int32 Diff = FChar::ToUnsigned(FChar::ToLower(CharA)) - FChar::ToUnsigned(FChar::ToLower(CharB)))
			{
				return Diff;
			}
		}
		else
		{
			return FChar::ToUnsigned(CharA) - FChar::ToUnsigned(CharB);
		}
 
		StringA++;
		StringB++;
	}
}
	
} // namespace UE::ComparisonUtility
