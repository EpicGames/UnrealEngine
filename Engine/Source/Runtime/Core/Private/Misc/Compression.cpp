// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/Compression.h"
#include "Misc/AssertionMacros.h"
#include "HAL/UnrealMemory.h"
#include "Logging/LogMacros.h"
#include "Misc/Parse.h"
#include "Misc/ScopeExit.h"
#include "Misc/ScopeLock.h"
#include "Misc/CommandLine.h"
#include "Stats/Stats.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CompressedGrowableBuffer.h"
#include "Misc/ICompressionFormat.h"

#include "Misc/MemoryReadStream.h"

#include "Compression/OodleDataCompression.h"
#include "Compression/CompressionUtil.h" // for extra logging only

// #include "TargetPlatformBase.h"
THIRD_PARTY_INCLUDES_START
#include "zlib.h"
THIRD_PARTY_INCLUDES_END

THIRD_PARTY_INCLUDES_START
#define LZ4_HC_STATIC_LINKING_ONLY
#include "Compression/lz4hc.h"
THIRD_PARTY_INCLUDES_END

DECLARE_LOG_CATEGORY_EXTERN(LogCompression, Log, All);
DEFINE_LOG_CATEGORY(LogCompression);

DECLARE_STATS_GROUP( TEXT( "Compression" ), STATGROUP_Compression, STATCAT_Advanced );

PRAGMA_DISABLE_UNSAFE_TYPECAST_WARNINGS

// static class members for registered plugin/module compression formats :
TMap<FName, struct ICompressionFormat*> FCompression::CompressionFormats;
FCriticalSection FCompression::CompressionFormatsCriticalSection;


static void *zalloc(void *opaque, unsigned int size, unsigned int num)
{
	return FMemory::Malloc(size * num);
}

static void zfree(void *opaque, void *p)
{
	FMemory::Free(p);
}

static const uint32 appZLIBVersion()
{
	return uint32(ZLIB_VERNUM);
}

static uint32 appGZIPVersion()
{
	return uint32(ZLIB_VERNUM); // we use zlib library for gzip
}

// Convert ECompressionFlags to the equivalent zlib compression level.
static int GetZlibCompressionLevel(const ECompressionFlags CompressionFlags)
{
	return (CompressionFlags & COMPRESS_BiasSize) ? Z_BEST_COMPRESSION
		: (CompressionFlags & COMPRESS_BiasSpeed) ? Z_BEST_SPEED
		: Z_DEFAULT_COMPRESSION;
}

static int GetLZ4HCCompressionLevel(const ECompressionFlags CompressionFlags)
{
	return (CompressionFlags & COMPRESS_BiasSize) ? LZ4HC_CLEVEL_MAX
		: (CompressionFlags & COMPRESS_BiasSpeed) ? LZ4HC_CLEVEL_MIN
		: LZ4HC_CLEVEL_DEFAULT;
}

/**
 * Thread-safe abstract compression routine. Compresses memory from uncompressed buffer and writes it to compressed
 * buffer. Updates CompressedSize with size of compressed data.
 *
 * @param	CompressedBuffer			Buffer compressed data is going to be written to
 * @param	CompressedSize	[in/out]	Size of CompressedBuffer, at exit will be size of compressed data
 * @param	UncompressedBuffer			Buffer containing uncompressed data
 * @param	UncompressedSize			Size of uncompressed data in bytes
 * @param	BitWindow					Bit window to use in compression
 * @return true if compression succeeds, false if it fails because CompressedBuffer was too small or other reasons
 */
static bool appCompressMemoryZLIB(void* CompressedBuffer, int64& CompressedSize, const void* UncompressedBuffer, int64 UncompressedSize, int32 BitWindow, int32 CompLevel)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(appCompressMemoryZLIB);
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("Compress Memory ZLIB"), STAT_appCompressMemoryZLIB, STATGROUP_Compression);

	ensureMsgf(CompLevel >= Z_DEFAULT_COMPRESSION, TEXT("CompLevel must be >= Z_DEFAULT_COMPRESSION"));
	ensureMsgf(CompLevel <= Z_BEST_COMPRESSION, TEXT("CompLevel must be <= Z_BEST_COMPRESSION"));

	CompLevel = FMath::Clamp(CompLevel, Z_DEFAULT_COMPRESSION, Z_BEST_COMPRESSION);

	// Compress data
	// If using the default Zlib bit window, use the zlib routines, otherwise go manual with deflate2
	if (BitWindow == 0 || BitWindow == DEFAULT_ZLIB_BIT_WINDOW)
	{
		if (!IntFitsIn<uLongf>(CompressedSize) ||
			!IntFitsIn<uLong>(UncompressedSize))
		{
			UE_LOGF(LogCompression, Error, "Requested a ZLIB compression that doesn't fit in uLong bits");
			return false;
		}
		uLongf ZCompressedSize = (uLongf)CompressedSize;
		uLong ZUncompressedSize = (uLong)UncompressedSize;
		bool bOperationSucceeded = compress2((uint8*)CompressedBuffer, &ZCompressedSize, (const uint8*)UncompressedBuffer, ZUncompressedSize, CompLevel) == Z_OK ? true : false;
		CompressedSize = ZCompressedSize;
		return bOperationSucceeded;
	}
	else
	{
		// Stream wants uInt not uLong
		z_stream stream;

		if (!IntFitsIn<decltype(stream.avail_out)>(CompressedSize) ||
			!IntFitsIn<decltype(stream.avail_in)>(UncompressedSize))
		{
			UE_LOGF(LogCompression, Error, "Requested a ZLIB compression that doesn't fit in uInt bits");
			return false;
		}

		stream.next_in = (Bytef*)UncompressedBuffer;
		stream.avail_in = (uInt)UncompressedSize;
		stream.next_out = (Bytef*)CompressedBuffer;
		stream.avail_out = (uInt)CompressedSize;
		stream.zalloc = &zalloc;
		stream.zfree = &zfree;
		stream.opaque = Z_NULL;

		bool bOperationSucceeded = false;

		if (ensure(Z_OK == deflateInit2(&stream, CompLevel, Z_DEFLATED, BitWindow, MAX_MEM_LEVEL, Z_DEFAULT_STRATEGY)))
		{
			if (ensure(Z_STREAM_END == deflate(&stream, Z_FINISH)))
			{
				CompressedSize = stream.total_out;
				if (ensure(Z_OK == deflateEnd(&stream)))
				{
					bOperationSucceeded = true;
				}
			}
			else
			{
				deflateEnd(&stream);
			}
		}

		return bOperationSucceeded;
	}
}

static bool appCompressMemoryGZIP(void* CompressedBuffer, int64& CompressedSize, const void* UncompressedBuffer, int64 UncompressedSize, int CompressionLevel)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(appCompressMemoryGZIP);
	DECLARE_SCOPE_CYCLE_COUNTER( TEXT( "Compress Memory GZIP" ), STAT_appCompressMemoryGZIP, STATGROUP_Compression );

	z_stream gzipstream;
	gzipstream.zalloc = &zalloc;
	gzipstream.zfree = &zfree;
	gzipstream.opaque = Z_NULL;

	if (!IntFitsIn<decltype(gzipstream.avail_in)>(UncompressedSize) ||
		!IntFitsIn<decltype(gzipstream.avail_out)>(CompressedSize))
	{
		UE_LOGF(LogCompression, Error, "Requested a ZLIB compression that doesn't fit in uInt bits");
		return false;
	}

	// Setup input buffer
	gzipstream.next_in = (uint8*)UncompressedBuffer;
	gzipstream.avail_in = UncompressedSize;

	// Init deflate settings to use GZIP
	int windowsBits = 15;
	int GZIP_ENCODING = 16;
	int InitError = deflateInit2(
		&gzipstream,
		CompressionLevel,
		Z_DEFLATED,
		windowsBits | GZIP_ENCODING,
		MAX_MEM_LEVEL,
		Z_DEFAULT_STRATEGY
	);
	if (InitError != Z_OK)
	{
		UE_LOGF(LogCompression, Error, "Failed to init ZLIB - error code %d", InitError);
		return false;
	}
	ON_SCOPE_EXIT
	{
		int EndError = deflateEnd(&gzipstream);
		if (EndError != Z_OK)
		{
			UE_LOGF(LogCompression, Warning, "Failed to shut down ZLIB - error code %d", EndError);
		}
	};

	// Setup output buffer
	const unsigned long GzipHeaderLength = 12;
	// This is how much memory we may need, however the consumer is allocating memory for us without knowing the required length.
	//unsigned long CompressedMaxSize = deflateBound(&gzipstream, gzipstream.avail_in) + GzipHeaderLength;
	gzipstream.next_out = (uint8*)CompressedBuffer;
	gzipstream.avail_out = CompressedSize;

	int Status = 0;
	while ((Status = deflate(&gzipstream, Z_FINISH)) == Z_OK);

	// Propagate compressed size from intermediate variable back into out variable.
	CompressedSize = gzipstream.total_out;

	bool bOperationSucceeded = (Status == Z_STREAM_END);
	return bOperationSucceeded;
}

static int appCompressMemoryBoundGZIP(int32 UncompressedSize)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(appCompressMemoryBoundGZIP);
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("Compress Memory Bound GZIP"), STAT_appCompressMemoryBoundGZIP, STATGROUP_Compression);
	z_stream gzipstream;
	gzipstream.zalloc = &zalloc;
	gzipstream.zfree = &zfree;
	gzipstream.opaque = Z_NULL;
	// Init deflate settings to use GZIP
	int windowsBits = 15;
	int GZIP_ENCODING = 16;
	deflateInit2(
		&gzipstream,
		Z_DEFAULT_COMPRESSION,
		Z_DEFLATED,
		windowsBits | GZIP_ENCODING,
		MAX_MEM_LEVEL,
		Z_DEFAULT_STRATEGY);
	// Return required size
	const unsigned long GzipHeaderLength = 12;
	int RequiredSize = deflateBound(&gzipstream, UncompressedSize) + GzipHeaderLength;
	deflateEnd(&gzipstream);
	return RequiredSize;
}

/**
 * Thread-safe abstract compression routine. Compresses memory from uncompressed buffer and writes it to compressed
 * buffer. Updates CompressedSize with size of compressed data.
 *
 * @param	UncompressedBuffer			Buffer containing uncompressed data
 * @param	UncompressedSize			Size of uncompressed data in bytes
 * @param	CompressedBuffer			Buffer compressed data is going to be read from
 * @param	CompressedSize				Size of CompressedBuffer data in bytes
 * @return true if compression succeeds, false if it fails because CompressedBuffer was too small or other reasons
 */
static bool appUncompressMemoryGZIP(void* UncompressedBuffer, int64 UncompressedSize, const void* CompressedBuffer, int64 CompressedSize)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(appUncompressMemoryGZIP);
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("Uncompress Memory GZIP"), STAT_appUncompressMemoryGZIP, STATGROUP_Compression);

	if (!IntFitsIn<decltype(z_stream::avail_in)>(UncompressedSize) ||
		!IntFitsIn<decltype(z_stream::avail_out)>(CompressedSize))
	{
		UE_LOGF(LogCompression, Error, "GZIP compression: can't fit in unsigned long: 0x%llx or 0x%llx", UncompressedSize, CompressedSize);
		return false;
	}

	// Zlib wants to use unsigned long.
	unsigned long ZCompressedSize = CompressedSize;
	unsigned long ZUncompressedSize = UncompressedSize;

	z_stream stream;
	stream.zalloc = &zalloc;
	stream.zfree = &zfree;
	stream.opaque = Z_NULL;
	stream.next_in = (uint8*)CompressedBuffer;
	stream.avail_in = ZCompressedSize;
	stream.next_out = (uint8*)UncompressedBuffer;
	stream.avail_out = ZUncompressedSize;

	int32 Result = inflateInit2(&stream, 16 + MAX_WBITS);

	if (Result != Z_OK)
		return false;

	// Uncompress data.
	Result = inflate(&stream, Z_FINISH);
	if (Result == Z_STREAM_END)
	{
		ZUncompressedSize = stream.total_out;
	}

	int32 EndResult = inflateEnd(&stream);
	if (Result >= Z_OK)
	{
		Result = EndResult;
	}

	// These warnings will be compiled out in shipping.
	UE_CLOGF(Result == Z_MEM_ERROR, LogCompression, Warning, "appUncompressMemoryGZIP failed: Error: Z_MEM_ERROR, not enough memory!");
	UE_CLOGF(Result == Z_BUF_ERROR, LogCompression, Warning, "appUncompressMemoryGZIP failed: Error: Z_BUF_ERROR, not enough room in the output buffer!");
	UE_CLOGF(Result == Z_DATA_ERROR, LogCompression, Warning, "appUncompressMemoryGZIP failed: Error: Z_DATA_ERROR, input data was corrupted or incomplete!");

	bool bOperationSucceeded = (Result == Z_OK);

	// Sanity check to make sure we uncompressed as much data as we expected to.
	if (UncompressedSize != ZUncompressedSize)
	{
		UE_LOG(LogCompression, Warning, TEXT("appUncompressMemoryGZIP failed: Mismatched uncompressed size. Expected: %" INT64_FMT ", Got:%ld. Result: %d"), UncompressedSize, ZUncompressedSize, Result);
		bOperationSucceeded = false;
	}
	return bOperationSucceeded;
}

/**
 * Thread-safe abstract compression routine. Compresses memory from uncompressed buffer and writes it to compressed
 * buffer. Updates CompressedSize with size of compressed data.
 *
 * @param	UncompressedBuffer			Buffer containing uncompressed data
 * @param	UncompressedSize			Size of uncompressed data in bytes
 * @param	CompressedBuffer			Buffer compressed data is going to be read from
 * @param	CompressedSize				Size of CompressedBuffer data in bytes
 * @return true if compression succeeds, false if it fails because CompressedBuffer was too small or other reasons
 */
static bool appUncompressMemoryZLIB( void* UncompressedBuffer, int64 UncompressedSize, const void* CompressedBuffer, int64 CompressedSize, int32 BitWindow )
{
	TRACE_CPUPROFILER_EVENT_SCOPE(appUncompressMemoryZLIB);
	DECLARE_SCOPE_CYCLE_COUNTER( TEXT( "Uncompress Memory ZLIB" ), STAT_appUncompressMemoryZLIB, STATGROUP_Compression );

	if (!IntFitsIn<decltype(z_stream::avail_in)>(UncompressedSize) ||
		!IntFitsIn<decltype(z_stream::avail_out)>(CompressedSize))
	{
		UE_LOGF(LogCompression, Error, "ZLIB compression: can't fit in unsigned long: 0x%llx or 0x%llx", UncompressedSize, CompressedSize);
		return false;
	}

	// Zlib wants to use unsigned long.
	unsigned long ZCompressedSize	= CompressedSize;
	unsigned long ZUncompressedSize	= UncompressedSize;
	
	z_stream stream;
	stream.zalloc = &zalloc;
	stream.zfree = &zfree;
	stream.opaque = Z_NULL;
	stream.next_in = (uint8*)CompressedBuffer;
	stream.avail_in = ZCompressedSize;
	stream.next_out = (uint8*)UncompressedBuffer;
	stream.avail_out = ZUncompressedSize;

	if (BitWindow == 0)
	{
		BitWindow = DEFAULT_ZLIB_BIT_WINDOW;
	}

	int32 Result = inflateInit2(&stream, BitWindow);

	if(Result != Z_OK)
		return false;

	// Uncompress data.
	Result = inflate(&stream, Z_FINISH);
	if(Result == Z_STREAM_END)
	{
		ZUncompressedSize = stream.total_out;
	}

	int32 EndResult = inflateEnd(&stream);
	if (Result >= Z_OK)
	{
		Result = EndResult;
	}

	// These warnings will be compiled out in shipping.
	UE_CLOGF(Result == Z_MEM_ERROR, LogCompression, Warning, "appUncompressMemoryZLIB failed: Error: Z_MEM_ERROR, not enough memory! (%ls)", UTF8_TO_TCHAR(stream.msg));
	UE_CLOGF(Result == Z_BUF_ERROR, LogCompression, Warning, "appUncompressMemoryZLIB failed: Error: Z_BUF_ERROR, not enough room in the output buffer! (%ls)", UTF8_TO_TCHAR(stream.msg));
	UE_CLOGF(Result == Z_DATA_ERROR, LogCompression, Warning, "appUncompressMemoryZLIB failed: Error: Z_DATA_ERROR, input data was corrupted or incomplete! (%ls)", UTF8_TO_TCHAR(stream.msg));

	bool bOperationSucceeded = (Result == Z_OK);

	// Sanity check to make sure we uncompressed as much data as we expected to.
	if( UncompressedSize != ZUncompressedSize )
	{
		UE_LOG( LogCompression, Warning, TEXT("appUncompressMemoryZLIB failed: Mismatched uncompressed size. Expected: %" INT64_FMT ", Got:%ld. Result: %d"), UncompressedSize, ZUncompressedSize, Result );
		bOperationSucceeded = false;
	}
	return bOperationSucceeded;
}

static bool appUncompressMemoryStreamZLIB(void* UncompressedBuffer, int64 UncompressedSize, IMemoryReadStream* Stream, int64 StreamOffset, int64 CompressedSize, int32 BitWindow)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(appUncompressMemoryStreamZLIB);
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("Uncompress Memory ZLIB"), STAT_appUncompressMemoryZLIB, STATGROUP_Compression);

	int64 ChunkOffset = 0;
	int64 ChunkSize = 0;
	const void* ChunkMemory = Stream->Read(ChunkSize, StreamOffset + ChunkOffset, CompressedSize);
	ChunkOffset += ChunkSize;

	if (!IntFitsIn<decltype(z_stream::avail_in)>(ChunkSize) ||
		!IntFitsIn<decltype(z_stream::avail_out)>(UncompressedSize))
	{
		UE_LOGF(LogCompression, Error, "ZLIB compression: can't fit in unsigned long: 0x%llx or 0x%llx", UncompressedSize, ChunkSize);
		return false;
	}

	z_stream stream;
	stream.zalloc = &zalloc;
	stream.zfree = &zfree;
	stream.opaque = Z_NULL;
	stream.next_in = (uint8*)ChunkMemory;
	stream.avail_in = ChunkSize;
	stream.next_out = (uint8*)UncompressedBuffer;
	stream.avail_out = UncompressedSize;

	if (BitWindow == 0)
	{
		BitWindow = DEFAULT_ZLIB_BIT_WINDOW;
	}

	int32 Result = inflateInit2(&stream, BitWindow);
	if (Result != Z_OK)
		return false;

	while (Result == Z_OK)
	{
		if (stream.avail_in == 0u)
		{
			ChunkMemory = Stream->Read(ChunkSize, StreamOffset + ChunkOffset, CompressedSize - ChunkOffset);
			ChunkOffset += ChunkSize;
			check(ChunkOffset <= CompressedSize);

			if (!IntFitsIn<decltype(z_stream::avail_in)>(ChunkSize))
			{
				UE_LOGF(LogCompression, Error, "ZLIB compression: can't fit in unsigned long: 0x%llx", ChunkSize);
				return false;
			}

			stream.next_in = (uint8*)ChunkMemory;
			stream.avail_in = ChunkSize;
		}

		Result = inflate(&stream, Z_SYNC_FLUSH);
	}

	int32 EndResult = inflateEnd(&stream);
	if (Result >= Z_OK)
	{
		Result = EndResult;
	}

	// These warnings will be compiled out in shipping.
	UE_CLOGF(Result == Z_MEM_ERROR, LogCompression, Warning, "appUncompressMemoryStreamZLIB failed: Error: Z_MEM_ERROR, not enough memory!");
	UE_CLOGF(Result == Z_BUF_ERROR, LogCompression, Warning, "appUncompressMemoryStreamZLIB failed: Error: Z_BUF_ERROR, not enough room in the output buffer!");
	UE_CLOGF(Result == Z_DATA_ERROR, LogCompression, Warning, "appUncompressMemoryStreamZLIB failed: Error: Z_DATA_ERROR, input data was corrupted or incomplete!");

	bool bOperationSucceeded = (Result == Z_OK);

	return bOperationSucceeded;
}

/** Time spent compressing data in cycles. */
TAtomic<uint64> FCompression::CompressorTimeCycles(0);
/** Number of bytes before compression.		*/
TAtomic<uint64> FCompression::CompressorSrcBytes(0);
/** Nubmer of bytes after compression.		*/
TAtomic<uint64> FCompression::CompressorDstBytes(0);

uint32 FCompression::GetCompressorVersion(FName FormatName)
{
	if (FormatName == NAME_None || FormatName == NAME_LZ4)
	{
		return 0;
	}
	else if (FormatName == NAME_Zlib || FormatName == NAME_Gzip)
	{
		return appZLIBVersion();
	}
	else
	{
		// let the format module compress it
		ICompressionFormat* Format = GetCompressionFormat(FormatName);
		if (Format)
		{
			return Format->GetVersion();
		}
	}

	return 0;
}

void FCompression::Init(void)
{
	FScopeLock Lock(&CompressionFormatsCriticalSection);
	
	// CompressionFormatInitOnFirstUseFromLock is safe to call multiple times
	FOodleDataCompression::CompressionFormatInitOnFirstUseFromLock();

	// OodleDataCompressionFormatInitOnFirstUseFromLock added it to the ModularFeatures list
}

ICompressionFormat* FCompression::GetCompressionFormat(FName FormatName, bool bErrorOnFailure)
{
	FScopeLock Lock(&CompressionFormatsCriticalSection);

	ICompressionFormat** ExistingFormat = CompressionFormats.Find(FormatName);
	if (ExistingFormat == nullptr)
	{
		if ( FormatName == NAME_Oodle )
		{
			// CompressionFormatInitOnFirstUseFromLock is a nop if already done
			// this should now usually be done by FCompression::Init from LaunchEngineLoop
			// but we need to keep this init-on-first-use here for the test harness
			// inside CompressionFormatsCriticalSection lock
			FOodleDataCompression::CompressionFormatInitOnFirstUseFromLock();
		}

		TArray<ICompressionFormat*> Features = IModularFeatures::Get().GetModularFeatureImplementations<ICompressionFormat>(COMPRESSION_FORMAT_FEATURE_NAME);

		for (ICompressionFormat* CompressionFormat : Features)
		{
			// is this format the right one?
			if (CompressionFormat->GetCompressionFormatName() == FormatName)
			{
				// remember it in our format map
				ExistingFormat = &CompressionFormats.Add(FormatName, CompressionFormat);
				break;
			}
		}

		if (ExistingFormat == nullptr)
		{
			if (bErrorOnFailure)
			{
				UE_LOGF(LogCompression, Error, "FCompression::GetCompressionFormat - Unable to find a module or plugin for compression format %ls", *FormatName.ToString());
			}
			else
			{
				UE_LOGF(LogCompression, Display, "FCompression::GetCompressionFormat - Unable to find a module or plugin for compression format %ls", *FormatName.ToString());
			}
			return nullptr;
		}
	}

	return *ExistingFormat;

}

FName FCompression::GetCompressionFormatFromDeprecatedFlags(ECompressionFlags Flags)
{
	switch (Flags & COMPRESS_DeprecatedFormatFlagsMask)
	{
		case COMPRESS_ZLIB_DEPRECATED:
			return NAME_Zlib;
		case COMPRESS_GZIP_DEPRECATED:
			return NAME_Gzip;
		// COMPRESS_Custom was a temporary solution to third party compression before we had plugins working, and it was only ever used with oodle, we just assume Oodle with Custom
		case COMPRESS_Custom_DEPRECATED:
			return NAME_Oodle;
		default:
			break;
	}

	return NAME_None;
}

bool FCompression::GetMaximumCompressedSize(FName FormatName, int64& OutMaxCompressedSize, int64 UncompressedSize, uintptr_t CompressionData)
{
	check(UncompressedSize >= 0);
	if (UncompressedSize < 0)
	{
		OutMaxCompressedSize = -1;
		UE_LOGF(LogCompression, Error, "Negative value passed to GetMaximumCompressedSize (0x%llx)", UncompressedSize);
		return false;
	}

	if (FormatName == NAME_None)
	{
		OutMaxCompressedSize = UncompressedSize;
		return true;
	}
	else if (FormatName == NAME_Oodle)
	{
		//	avoid calling CompressMemoryBound in the Decoder because it creates an ICompressionFormat for Oodle
		//	and initializes encoders (and also is a different value!)
		// This should be codec independent because it's referring to how much it gets compressed, not the buffer space needed
		// to compress... and we should be OK for overflow because this just means future oodle size checks will fail, not that
		// anything will get stomped.
		OutMaxCompressedSize = FOodleDataCompression::GetMaximumCompressedSize(UncompressedSize);
		return true;
	}

	// If we don't have anything better to use then we just use the compressed buffer size, which is almost certainly
	// too big but also all we can go on.
	return CompressMemoryBound(FormatName, OutMaxCompressedSize, UncompressedSize, CompressionData);
}

bool FCompression::CompressMemoryBound(FName FormatName, int64& OutBufferSizeRequired, int64 UncompressedSize, uintptr_t CompressionData)
{
	// Init to a garbage value so if they don't pay attention to the return value they 
	// crash allocating a massive buffer.
	OutBufferSizeRequired = -1;

	check(UncompressedSize >= 0);
	if (UncompressedSize < 0)
	{
		UE_LOGF(LogCompression, Error, "Negative value passed to CompressMemoryBound (0x%llx)", UncompressedSize);
		return false;
	}

	if (FormatName == NAME_None)
	{
		OutBufferSizeRequired = UncompressedSize;
		return true;
	}
	else if (FormatName == NAME_Zlib)
	{
		if (!IntFitsIn<uLong>(UncompressedSize))
		{
			UE_LOGF(LogCompression, Error, "Zlib doesn't support >32 bit sizes (0x%llx)", UncompressedSize);
			return false;
		}

		// Zlib's compressBounds gives a better (smaller) value, but only for the default bit window.
		if (CompressionData == 0 || CompressionData == DEFAULT_ZLIB_BIT_WINDOW)
		{
			OutBufferSizeRequired = compressBound(UncompressedSize);
		}
		else
		{
			// Calculate pessimistic bounds for compression. This value is calculated based on the algorithm used in deflate2.
			OutBufferSizeRequired = UncompressedSize + ((UncompressedSize + 7) >> 3) + ((UncompressedSize + 63) >> 6) + 5 + 6;
			if (OutBufferSizeRequired < 0)
			{
				UE_LOGF(LogCompression, Error, "Zlib CompressMemoryBound calculated negative value 0x%llx -> 0x%llx", UncompressedSize, OutBufferSizeRequired);
				return false;
			}
		}
		return true;
	}
	else if (FormatName == NAME_Gzip)
	{
		if (!IntFitsIn<int32>(UncompressedSize))
		{
			UE_LOGF(LogCompression, Error, "Gzip doesn't support >32 bit sizes (0x%llx)", UncompressedSize);
			return false;
		}

		OutBufferSizeRequired = appCompressMemoryBoundGZIP((int32)UncompressedSize);
		if (OutBufferSizeRequired < 0)
		{
			UE_LOGF(LogCompression, Error, "Gzip CompressMemoryBound calculated negative value 0x%llx -> 0x%llx", UncompressedSize, OutBufferSizeRequired);
			return false;
		}
		return true;
	}
	else if (FormatName == NAME_LZ4)
	{
		if (UncompressedSize > LZ4_MAX_INPUT_SIZE)
		{
			UE_LOGF(LogCompression, Error, "LZ4 doesn't support >32 bit sizes (0x%llx) max is 0x%x", UncompressedSize, LZ4_MAX_INPUT_SIZE);
			return false;
		}
		OutBufferSizeRequired = LZ4_compressBound((int)UncompressedSize);
		if (OutBufferSizeRequired < 0)
		{
			UE_LOGF(LogCompression, Error, "LZ4 CompressMemoryBound calculated negative value 0x%llx -> 0x%llx", UncompressedSize, OutBufferSizeRequired);
			return false;
		}
		return true;
	}

	ICompressionFormat* Format = GetCompressionFormat(FormatName);
	if (!Format)
	{
		return false;
	}

	if (!Format->GetCompressedBufferSize(OutBufferSizeRequired, UncompressedSize, CompressionData))
	{
		UE_LOGF(LogCompression, Error, "GetCompressedBufferSize for format %ls failed to return compression bound: check bits needed? (0x%llx)", *FormatName.ToString(), UncompressedSize);
		return false;
	}

	if (OutBufferSizeRequired < 0)
	{
		UE_LOGF(LogCompression, Error, "%ls CompressMemoryBound calculated negative value 0x%llx -> 0x%llx", *FormatName.ToString(), UncompressedSize, OutBufferSizeRequired);
		return false;
	}

	return true;
}

// 32 thunk to 64 bit.
int32 FCompression::CompressMemoryBound(FName FormatName, int32 UncompressedSize, ECompressionFlags Flags, int32 CompressionData)
{
	int64 BufferSizeNeeded = 0;
	bool bSucceeded = CompressMemoryBound(FormatName, BufferSizeNeeded, UncompressedSize, CompressionData);
	if (!bSucceeded ||
		!IntFitsIn<int32>(BufferSizeNeeded))
	{
		UE_LOGF(LogCompression, Fatal, "CompressMemoryBound failed, check sizes/format (%d, %ls)", UncompressedSize, *FormatName.ToString());
		return -1;
	}

	return (int32)BufferSizeNeeded;
}

// 32 thunk to 64 bit.
bool FCompression::CompressMemoryIfWorthDecompressing(FName FormatName, int32 MinBytesSaved, int32 MinPercentSaved, void* CompressedBuffer, int32& CompressedSize, const void* UncompressedBuffer, int32 UncompressedSize, ECompressionFlags Flags, int32 CompressionData)
{
	int64 CompressedSize64 = CompressedSize;
	bool bWasCompressed = false;
	bool bSucceeded = CompressMemoryIfWorthDecompressing(FormatName, bWasCompressed, MinBytesSaved, MinPercentSaved, CompressedBuffer, CompressedSize64, UncompressedBuffer, UncompressedSize, Flags, CompressionData);

	// Behavior change: compression failure now FATALs instead of returning "wasn't compressed" (in addition to the type narrowing now caught)
	if (!bSucceeded ||
		!IntFitsIn<int32>(CompressedSize64))
	{
		UE_LOGF(LogCompression, Fatal, "CompressMemoryIfWorthDecompressing failed, check sizes/format (%d, %ls)", UncompressedSize, *FormatName.ToString());
		return false;
	}

	CompressedSize = (int32)CompressedSize64;
	return bWasCompressed;
}

bool FCompression::CompressMemoryIfWorthDecompressing(FName FormatName, bool& bOutWasCompressed, int64 MinBytesSaved, int32 MinPercentSaved, void* CompressedBuffer, int64& CompressedSize, const void* UncompressedBuffer, int64 UncompressedSize, ECompressionFlags Flags, uintptr_t CompressionData)
{
	// init to false so that if they ignore the return they just pass it uncompressed.
	bOutWasCompressed = false;
	if (UncompressedSize < -1 ||
		CompressedSize < -1)
	{
		UE_LOGF(LogCompression, Error, "Negative value passed to CompressMemoryIfWorthDecompressing (0x%llx / 0x%llx)", UncompressedSize, CompressedSize);
		return false;
	}

	// returns false if we could compress,
	//	but it's not worth the time to decompress
	//	you should store the data uncompressed instead

	if ( UncompressedSize <= MinBytesSaved )
	{
		// if input size is smaller than the number of bytes we need to save
		// no need to even try encoding
		// also saves encode time
		// NOTE : this check applies even for compressor who say "bNeedsWorthItCheck = false" , eg. Oodle
		bOutWasCompressed = false;
		return true;
	}
	
	bool bNeedsWorthItCheck = false;
	
	if (FormatName == NAME_Oodle)
	{
		// Oodle does its own internal "worth it" check
		bNeedsWorthItCheck = false;
	}
	else if (FormatName == NAME_Zlib || FormatName == NAME_Gzip || FormatName == NAME_LZ4)
	{
		bNeedsWorthItCheck = true;
	}
	else
	{
		ICompressionFormat* Format = GetCompressionFormat(FormatName);

		if ( ! Format )
		{
			return false;
		}

		bNeedsWorthItCheck = ! Format->DoesOwnWorthDecompressingCheck();
	}

	bool bCompressSucceeded = CompressMemory(FormatName,CompressedBuffer,CompressedSize,UncompressedBuffer,UncompressedSize,Flags,CompressionData);

	if ( ! bCompressSucceeded )
	{
		// compression actually failed
		return false;
	}

	if ( ! bNeedsWorthItCheck )
	{
		// ICompressionFormat does own "worth it" check, don't do our own
		// do check for expansion because that's how they signal "not worth it"
		//	(CompressMemory is not allowed to return false)
		bOutWasCompressed = ( CompressedSize < UncompressedSize );
		return true;
	}

	// we got compression, but do we want it ?
	
	// check if the decode time on load is worth the size savings
	// Oodle uses much more sophisticated models for this
	// here we replicate the Pak file logic :

	// must save at least MinBytesSaved regardless of percentage (for small files)
	// also checks CompressedSize >= UncompressedSize
	int64 BytesSaved = UncompressedSize - CompressedSize;
	if ( BytesSaved < MinBytesSaved )
	{
		bOutWasCompressed = false;
		return true;
	}

	// Check the saved compression ratio, if it's too low just store uncompressed. 
	// For example, saving 64 KB per 1 MB is about 6%.
	bOutWasCompressed = BytesSaved * 100 >= UncompressedSize * MinPercentSaved;
	return true;
}

bool FCompression::CompressMemory(FName FormatName, void* CompressedBuffer, int32& CompressedSize, const void* UncompressedBuffer, int32 UncompressedSize, ECompressionFlags Flags, int32 CompressionData)
{
	int64 CompressedSize64 = CompressedSize;
	bool bSucceeded = false;
	UE_AUTORTFM_OPEN
	{
		bSucceeded = CompressMemory(FormatName, CompressedBuffer, CompressedSize64, UncompressedBuffer, UncompressedSize, Flags, CompressionData);
	};
	if (!IntFitsIn<int32>(CompressedSize64))
	{
		UE_LOGF(LogCompression, Error, "Compressing 32 bit memory size ended up a 64 bit size! %d -> %lld", UncompressedSize, CompressedSize64);
		return false;
	}
	CompressedSize = (int32)CompressedSize64;
	return bSucceeded;
}

bool FCompression::CompressMemory(FName FormatName, void* CompressedBuffer, int64& CompressedSize, const void* UncompressedBuffer, int64 UncompressedSize, ECompressionFlags Flags, uintptr_t CompressionData)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FCompression::CompressMemory);
	uint64 CompressorStartTime = FPlatformTime::Cycles64();

	bool bCompressSucceeded = false;

	if (FormatName == NAME_Zlib)
	{
		// hardcoded zlib
		bCompressSucceeded = appCompressMemoryZLIB(CompressedBuffer, CompressedSize, UncompressedBuffer, UncompressedSize, CompressionData, GetZlibCompressionLevel(Flags));
	}
	else if (FormatName == NAME_Gzip)
	{
		// hardcoded gzip
		bCompressSucceeded = appCompressMemoryGZIP(CompressedBuffer, CompressedSize, UncompressedBuffer, UncompressedSize, GetZlibCompressionLevel(Flags));
	}
	else if (FormatName == NAME_LZ4)
	{
		// hardcoded lz4
		if (UncompressedSize > LZ4_MAX_INPUT_SIZE)
		{
			UE_LOGF(LogCompression, Error, "LZ4 can't compress larger than 0x%x (passed 0x%llx)", LZ4_MAX_INPUT_SIZE, UncompressedSize);
			return false;
		}

		CompressedSize = LZ4_compress_HC((const char*)UncompressedBuffer, (char*)CompressedBuffer, UncompressedSize, CompressedSize, GetLZ4HCCompressionLevel(Flags));
		bCompressSucceeded = CompressedSize > 0;
	}
	else
	{
		// let the format module compress it
		// Oodle will make the OodleCompressionFormat here
		ICompressionFormat* Format = GetCompressionFormat(FormatName);
		if (Format)
		{
			bCompressSucceeded = Format->Compress(CompressedBuffer, CompressedSize, UncompressedBuffer, UncompressedSize, CompressionData, Flags);
		}
	}

	// Keep track of compression time and stats.
	CompressorTimeCycles += FPlatformTime::Cycles64() - CompressorStartTime;
	if( bCompressSucceeded )
	{
		CompressorSrcBytes += UncompressedSize;
		CompressorDstBytes += CompressedSize;
	}

	return bCompressSucceeded;
}

#define ZLIB_DERIVEDDATA_VER TEXT("9810EC9C5D34401CBD57AA3852417A6C")
#define GZIP_DERIVEDDATA_VER TEXT("FB2181277DF44305ABBE03FD1751CBDE")


FString FCompression::GetCompressorDDCSuffix(FName FormatName)
{
	FString DDCSuffix = FString::Printf(TEXT("%s_VER%d_"), *FormatName.ToString(), FCompression::GetCompressorVersion(FormatName));

	if (FormatName == NAME_None || FormatName == NAME_LZ4 )
	{
		// nothing
	}
	else if (FormatName == NAME_Zlib)
	{
		// hardcoded zlib
		DDCSuffix += ZLIB_DERIVEDDATA_VER;
	}
	else if (FormatName == NAME_Gzip)
	{
		DDCSuffix += GZIP_DERIVEDDATA_VER;
	}
	else
	{
		// let the format module compress it
		ICompressionFormat* Format = GetCompressionFormat(FormatName);
		if (Format)
		{
			DDCSuffix += Format->GetDDCKeySuffix();
		}
	}
	
	return DDCSuffix;
}

DECLARE_FLOAT_ACCUMULATOR_STAT(TEXT("Uncompressor total time"),STAT_UncompressorTime,STATGROUP_Compression);

bool FCompression::UncompressMemory(FName FormatName, void* UncompressedBuffer, int64 UncompressedSize, const void* CompressedBuffer, int64 CompressedSize, ECompressionFlags Flags, uintptr_t CompressionData)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FCompression::UncompressMemory);
	SCOPED_NAMED_EVENT(FCompression_UncompressMemory, FColor::Cyan);
	// Keep track of time spent uncompressing memory.
	STAT(double UncompressorStartTime = FPlatformTime::Seconds();)
	
	bool bUncompressSucceeded = false;

	if (FormatName == NAME_Zlib)
	{
		// hardcoded zlib
		bUncompressSucceeded = appUncompressMemoryZLIB(UncompressedBuffer, UncompressedSize, CompressedBuffer, CompressedSize, CompressionData);
	}
	else if (FormatName == NAME_Gzip)
	{
		// hardcoded gzip
		bUncompressSucceeded = appUncompressMemoryGZIP(UncompressedBuffer, UncompressedSize, CompressedBuffer, CompressedSize);
	}
	else if (FormatName == NAME_LZ4)
	{
		if (!IntFitsIn<int>(UncompressedSize) ||
			!IntFitsIn<int>(CompressedSize))
		{
			UE_LOGF(LogCompression, Error, "LZ4 can't fit in int: 0x%llx or 0x%llx", CompressedSize, UncompressedSize);
			return false;
		}

		// hardcoded lz4
		bUncompressSucceeded = LZ4_decompress_safe((const char*)CompressedBuffer, (char*)UncompressedBuffer, CompressedSize, UncompressedSize) > 0;
	}
	else if (FormatName == NAME_Oodle)
	{
		// hardcoded Oodle
		// can decode Oodle data without creating Oodle ICompressionFormat
		bUncompressSucceeded = FOodleDataCompression::Decompress(UncompressedBuffer,UncompressedSize,CompressedBuffer,CompressedSize);
	}
	else
	{
		// let the format module compress it
		ICompressionFormat* Format = GetCompressionFormat(FormatName);
		if (Format)
		{
			bUncompressSucceeded = Format->Uncompress(UncompressedBuffer, UncompressedSize, CompressedBuffer, CompressedSize, CompressionData);
		}
	}

	if (!bUncompressSucceeded)
	{
		// This is only to skip serialization errors caused by asset corruption 
		// that can be fixed during re-save, should never be disabled by default!
		static struct FFailOnUncompressErrors
		{
			bool Value;
			FFailOnUncompressErrors()
				: Value(true) // fail by default
			{
				// very early decodes of first paks could be before this config is loaded
				if ( GConfig )
				{
					GConfig->GetBool(TEXT("Core.System"), TEXT("FailOnUncompressErrors"), Value, GEngineIni);
				}
			}
		} FailOnUncompressErrors;
		if (!FailOnUncompressErrors.Value)
		{
			bUncompressSucceeded = true;
		}
		// Always log an error
		UE_LOG(LogCompression, Error, TEXT("FCompression::UncompressMemory - Failed to uncompress memory (%" INT64_FMT "/%" INT64_FMT ") from address %p using format %s, this may indicate the asset is corrupt!"), CompressedSize, UncompressedSize, CompressedBuffer, *FormatName.ToString());
		// this extra logging is added to understand shader decompression errors, see UE-159777. However in unrelated
		// corruption issues this gets hit a lot causing massive log sizes. Since for UE-159777 we crash afterwards, we
		// can safely limit to one instance for the purpose of diagnosing this.
		static std::atomic_int32_t has_logged = 0;
		bool bAllowLog = has_logged.exchange(1, std::memory_order_relaxed) == 0;
		const int32 MaxSizeToLogOut = 16384;
		if (bAllowLog && FormatName == NAME_Oodle && CompressedSize <= MaxSizeToLogOut)
		{
			UE_LOG(LogCompression, Error, TEXT("FCompression::UncompressMemory - Logging compressed data (%" INT64_FMT " bytes) as a hex dump for investigation"), CompressedSize);
			FCompressionUtil::LogHexDump(reinterpret_cast<const uint8*>(CompressedBuffer), CompressedSize, 0, CompressedSize);
		}
	}

#if	STATS
	if (FThreadStats::IsThreadingReady())
	{
		INC_FLOAT_STAT_BY( STAT_UncompressorTime, (float)(FPlatformTime::Seconds() - UncompressorStartTime) )
	}
#endif // STATS
	
	return bUncompressSucceeded;
}

bool FCompression::UncompressMemoryStream(FName FormatName, void* UncompressedBuffer, int64 UncompressedSize, IMemoryReadStream* Stream, int64 StreamOffset, int64 CompressedSize, ECompressionFlags Flags, uintptr_t CompressionData)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FCompression::UncompressMemoryStream);

	int64 ContiguousChunkSize = 0;
	const void* ContiguousMemory = Stream->Read(ContiguousChunkSize, StreamOffset, CompressedSize);
	bool bUncompressResult = false;
	if (ContiguousChunkSize >= CompressedSize)
	{
		// able to map entire memory stream as contiguous buffer, use default uncompress here to take advantage of possible platform optimization
		bUncompressResult = UncompressMemory(FormatName, UncompressedBuffer, UncompressedSize, ContiguousMemory, CompressedSize, Flags, CompressionData);
	}
	else if (FormatName == NAME_Zlib)
	{
		SCOPED_NAMED_EVENT(FCompression_UncompressMemoryStream, FColor::Cyan);
		// Keep track of time spent uncompressing memory.
		STAT(double UncompressorStartTime = FPlatformTime::Seconds();)
		// ZLib supports streaming implementation for non-contiguous buffers
		bUncompressResult = appUncompressMemoryStreamZLIB(UncompressedBuffer, UncompressedSize, Stream, StreamOffset, CompressedSize, CompressionData);
#if	STATS
		if (FThreadStats::IsThreadingReady())
		{
			INC_FLOAT_STAT_BY(STAT_UncompressorTime, (float)(FPlatformTime::Seconds() - UncompressorStartTime))
		}
#endif // STATS
	}
	else
	{
		// need to allocate temp memory to create contiguous buffer for default uncompress
		void* TempMemory = FMemory::Malloc(CompressedSize);
		Stream->CopyTo(TempMemory, StreamOffset, CompressedSize);
		bUncompressResult = UncompressMemory(FormatName, UncompressedBuffer, UncompressedSize, TempMemory, CompressedSize, Flags, CompressionData);
		FMemory::Free(TempMemory);
	}

	return bUncompressResult;
}

/*-----------------------------------------------------------------------------
	FCompressedGrowableBuffer.
-----------------------------------------------------------------------------*/

FCompressedGrowableBuffer::FCompressedGrowableBuffer( int32 InMaxPendingBufferSize, FName InCompressionFormat, ECompressionFlags InCompressionFlags )
:	MaxPendingBufferSize( InMaxPendingBufferSize )
,	CompressionFormat( InCompressionFormat )
,	CompressionFlags(InCompressionFlags)
,	CurrentOffset( 0 )
,	NumEntries( 0 )
,	DecompressedBufferBookKeepingInfoIndex( INDEX_NONE )
{
	PendingCompressionBuffer.Empty( MaxPendingBufferSize );
}

/**
 * Locks the buffer for reading. Needs to be called before calls to Access and needs
 * to be matched up with Unlock call.
 */
void FCompressedGrowableBuffer::Lock()
{
	check( DecompressedBuffer.Num() == 0 );
}

/**
 * Unlocks the buffer and frees temporary resources used for accessing.
 */
void FCompressedGrowableBuffer::Unlock()
{
	DecompressedBuffer.Empty();
	DecompressedBufferBookKeepingInfoIndex = INDEX_NONE;
}

/**
 * Appends passed in data to the buffer. The data needs to be less than the max
 * pending buffer size. The code will assert on this assumption.
 *
 * @param	Data	Data to append
 * @param	Size	Size of data in bytes.
 * @return	Offset of data, used for retrieval later on
 */
int32 FCompressedGrowableBuffer::Append( void* Data, int32 Size )
{
	check( DecompressedBuffer.Num() == 0 );
	check( Size <= MaxPendingBufferSize );
	NumEntries++;

	// Data does NOT fit into pending compression buffer. Compress existing data 
	// and purge buffer.
	if( MaxPendingBufferSize - PendingCompressionBuffer.Num() < Size )
	{
		// Allocate temporary buffer to hold compressed data. It is bigger than the uncompressed size as
		// compression is not guaranteed to create smaller data and we don't want to handle that case so 
		// we simply assert if it doesn't fit. For all practical purposes this works out fine and is what
		// other code in the engine does as well.
		int32 CompressedSize = MaxPendingBufferSize * 4 / 3;
		void* TempBuffer = FMemory::Malloc( CompressedSize );

		// Compress the memory. CompressedSize is [in/out]
		verify( FCompression::CompressMemory( CompressionFormat, TempBuffer, CompressedSize, PendingCompressionBuffer.GetData(), PendingCompressionBuffer.Num(), CompressionFlags ) );

		// Append the compressed data to the compressed buffer and delete temporary data.
		int32 StartIndex = CompressedBuffer.AddUninitialized( CompressedSize );
		FMemory::Memcpy( &CompressedBuffer[StartIndex], TempBuffer, CompressedSize );
		FMemory::Free( TempBuffer );

		// Keep track of book keeping info for later access to data.
		FBufferBookKeeping Info;
		Info.CompressedOffset = StartIndex;
		Info.CompressedSize = CompressedSize;
		Info.UncompressedOffset = CurrentOffset - PendingCompressionBuffer.Num();
		Info.UncompressedSize = PendingCompressionBuffer.Num();
		BookKeepingInfo.Add( Info ); 

		// Resize & empty the pending buffer to the default state.
		PendingCompressionBuffer.Empty( MaxPendingBufferSize );
	}

	// Appends the data to the pending buffer. The pending buffer is compressed
	// as needed above.
	int32 StartIndex = PendingCompressionBuffer.AddUninitialized( Size );
	FMemory::Memcpy( &PendingCompressionBuffer[StartIndex], Data, Size );

	// Return start offset in uncompressed memory.
	int32 StartOffset = CurrentOffset;
	CurrentOffset += Size;
	return StartOffset;
}

/**
 * Accesses the data at passed in offset and returns it. The memory is read-only and
 * memory will be freed in call to unlock. The lifetime of the data is till the next
 * call to Unlock, Append or Access
 *
 * @param	Offset	Offset to return corresponding data for
 */
void* FCompressedGrowableBuffer::Access( int32 Offset )
{
	void* UncompressedData = NULL;

	// Check whether the decompressed data is already cached.
	if( DecompressedBufferBookKeepingInfoIndex != INDEX_NONE )
	{
		const FBufferBookKeeping& Info = BookKeepingInfo[DecompressedBufferBookKeepingInfoIndex];
		// Cache HIT.
		if( (Info.UncompressedOffset <= Offset) && (Info.UncompressedOffset + Info.UncompressedSize > Offset) )
		{
			// Figure out index into uncompressed data and set it. DecompressionBuffer (return value) is going 
			// to be valid till the next call to Access or Unlock.
			int32 InternalOffset = Offset - Info.UncompressedOffset;
			UncompressedData = &DecompressedBuffer[InternalOffset];
		}
		// Cache MISS.
		else
		{
			DecompressedBufferBookKeepingInfoIndex = INDEX_NONE;
		}
	}

	// Traverse book keeping info till we find the matching block.
	if( UncompressedData == NULL )
	{
		for( int32 InfoIndex=0; InfoIndex<BookKeepingInfo.Num(); InfoIndex++ )
		{
			const FBufferBookKeeping& Info = BookKeepingInfo[InfoIndex];
			if( (Info.UncompressedOffset <= Offset) && (Info.UncompressedOffset + Info.UncompressedSize > Offset) )
			{
				// Found the right buffer, now decompress it.
				DecompressedBuffer.Empty( Info.UncompressedSize );
				DecompressedBuffer.AddUninitialized( Info.UncompressedSize );
				verify( FCompression::UncompressMemory( CompressionFormat, DecompressedBuffer.GetData(), Info.UncompressedSize, &CompressedBuffer[Info.CompressedOffset], Info.CompressedSize, CompressionFlags ) );

				// Figure out index into uncompressed data and set it. DecompressionBuffer (return value) is going 
				// to be valid till the next call to Access or Unlock.
				int32 InternalOffset = Offset - Info.UncompressedOffset;
				UncompressedData = &DecompressedBuffer[InternalOffset];	

				// Keep track of buffer index for the next call to this function.
				DecompressedBufferBookKeepingInfoIndex = InfoIndex;
				break;
			}
		}
	}

	// If we still haven't found the data it might be in the pending compression buffer.
	if( UncompressedData == NULL )
	{
		int32 UncompressedStartOffset = CurrentOffset - PendingCompressionBuffer.Num();
		if( (UncompressedStartOffset <= Offset) && (CurrentOffset > Offset) )
		{
			// Figure out index into uncompressed data and set it. PendingCompressionBuffer (return value) 
			// is going to be valid till the next call to Access, Unlock or Append.
			int32 InternalOffset = Offset - UncompressedStartOffset;
			UncompressedData = &PendingCompressionBuffer[InternalOffset];
		}
	}

	// Return value is only valid till next call to Access, Unlock or Append!
	check( UncompressedData );
	return UncompressedData;
}

bool FCompression::IsFormatValid(FName FormatName)
{
	//@todo Oodle make NAME_None a valid compressor
	// FormatName == NAME_None || 

	// build in formats are always valid
	if ( FormatName == NAME_Zlib || FormatName == NAME_Gzip || FormatName == NAME_LZ4 || FormatName == NAME_Oodle)
	{
		return true;
	}

	// otherwise, if we can get the format class, we are good!
	return GetCompressionFormat(FormatName, false) != nullptr;
}

bool FCompression::VerifyCompressionFlagsValid(int32 InCompressionFlags)
{
	const int32 CompressionFlagsMask = COMPRESS_DeprecatedFormatFlagsMask | COMPRESS_OptionsFlagsMask | COMPRESS_ForPurposeMask;
	if (InCompressionFlags & (~CompressionFlagsMask))
	{
		return false;
	}
	// @todo: check the individual flags here
	return true;
}


/***********************
  Deprecated functions
***********************/
PRAGMA_RESTORE_UNSAFE_TYPECAST_WARNINGS
