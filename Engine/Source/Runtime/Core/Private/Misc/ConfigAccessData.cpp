// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/ConfigAccessData.h"

#if UE_WITH_CONFIG_TRACKING || WITH_EDITOR

#include "Misc/StringBuilder.h"

namespace UE::ConfigAccessTracking
{

FConfigAccessData::FConfigAccessData(ELoadType InLoadType, FNameEntryId InConfigPlatform, FNameEntryId InFileName)
	: ConfigPlatform(InConfigPlatform)
	, FileName(InFileName)
	, LoadType(InLoadType)
{
}

FConfigAccessData::FConfigAccessData(ELoadType InLoadType, FNameEntryId InConfigPlatform, FNameEntryId InFileName,
	FNameEntryId InSectionName, FMinimalName InValueName, const ITargetPlatform* InRequestingPlatform)
	: ConfigPlatform(InConfigPlatform)
	, FileName(InFileName)
	, SectionName(InSectionName)
	, ValueName(InValueName)
	, RequestingPlatform(InRequestingPlatform)
	, LoadType(InLoadType)
{
}

FConfigAccessData FConfigAccessData::GetFileOnlyData() const
{
	return FConfigAccessData(LoadType, ConfigPlatform, FileName);
}

FConfigAccessData FConfigAccessData::GetPathOnlyData() const
{
	return FConfigAccessData(LoadType, ConfigPlatform, FileName, SectionName, ValueName, nullptr);
}

FString FConfigAccessData::FullPathToString() const
{
	TStringBuilder<256> Out;
	AppendFullPath(Out);
	return FString(*Out);
}

void FConfigAccessData::AppendFullPath(FStringBuilderBase& Out) const
{
	if (LoadType == ELoadType::Uninitialized || FileName.IsNone())
	{
		Out << TEXTVIEW("<Invalid>");
		return;
	}

	Out << LexToString(LoadType) << TEXTVIEW(".");
	if (ConfigPlatform.IsNone())
	{
		Out << PlatformAgnosticName;
	}
	else
	{
		EscapeConfigTrackingTokenAppendString(GetConfigPlatform(), Out);
	}
	Out << TEXTVIEW(".");
	EscapeConfigTrackingTokenAppendString(GetFileName(), Out);
	if (!SectionName.IsNone())
	{
		Out << TEXTVIEW(":[");
		EscapeConfigTrackingTokenAppendString(GetSectionName(), Out);
		Out << TEXTVIEW("]");
		if (!ValueName.IsNone())
		{
			Out << TEXTVIEW(":");
			EscapeConfigTrackingTokenAppendString(GetValueName(), Out);
		}
	}
}

FConfigAccessData FConfigAccessData::Parse(FStringView Text)
{
	// ConfigSystem.<Editor>.../../../Engine/Config/ConsoleVariables.ini:Section:ValueName
	//   -> "ConfigSystem", "<Editor>", "../../../Engine/Config/ConsoleVariables.ini", "Section", "ValueName"
	// No tokens can contain a single colon, but they might contain double colon, which is the escape code for a single colon.
	// 3rd token might have dots, first two cannot.

	TStringBuilder<128> FullFilePath;
	TStringBuilder<64> SectionNameStr;
	TStringBuilder<64> ValueNameStr;
	FStringBuilderBase* TokenBuffer[] = { &FullFilePath, &SectionNameStr, &ValueNameStr };
	TArrayView<FStringBuilderBase*> ColonDelimitedTokens = TokenBuffer;
	TryTokenizeConfigTrackingString(Text, ColonDelimitedTokens);
	if (FullFilePath.Len() == 0)
	{
		return FConfigAccessData();
	}

	FStringView FullFilePathTokens[3];
	FullFilePathTokens[0] = FullFilePath;
	int32 NumFilePathTokens = 1;
	for (NumFilePathTokens = 1; NumFilePathTokens < UE_ARRAY_COUNT(FullFilePathTokens); ++NumFilePathTokens)
	{
		FStringView& Current = FullFilePathTokens[NumFilePathTokens - 1];
		FStringView& Next = FullFilePathTokens[NumFilePathTokens];
		int32 DotIndex = Current.Find(TEXTVIEW("."));
		if (DotIndex == INDEX_NONE)
		{
			break;
		}
		Next = Current.RightChop(DotIndex + 1);
		Current.LeftInline(DotIndex);
		if (Next.IsEmpty())
		{
			// break before incrementing NumFilePathTokens so that we do not count the empty Next as a token
			break;
		}
	}
	// FullFilePath is of the form 
	//   LoadType.Platform.ConfigName
	// Platform is <Editor> if the configfile was an editor config file rather than a platform-specific config file
	if (NumFilePathTokens < 3)
	{
		// Not a valid FConfigAccessData text; we always require all 3 of LoadType,Platform,ConfigName
		return FConfigAccessData();
	}

	FConfigAccessData Result;
	LexFromString(Result.LoadType, FullFilePathTokens[0]);
	if (Result.LoadType == ELoadType::Uninitialized)
	{
		return FConfigAccessData();
	}

	FName LocalConfigPlatform = FullFilePathTokens[1] == PlatformAgnosticName
		? NAME_None : FName(FullFilePathTokens[1]);
	FStringView SectionNameView(SectionNameStr);
	// We write out sectionnames with [] for readability; remove them if they exist.
	if (SectionNameView.StartsWith('[')) SectionNameView.RightChopInline(1);
	if (SectionNameView.EndsWith(']')) SectionNameView.LeftChopInline(1);
	FName LocalSectionName = SectionNameView.Len() == 0 ? NAME_None : FName(SectionNameView, NAME_NO_NUMBER_INTERNAL);
	FName LocalValueName = ValueNameStr.Len() == 0 ? NAME_None : FName(ValueNameStr);

	Result.ConfigPlatform = LocalConfigPlatform.GetComparisonIndex();
	Result.FileName = FName(FullFilePathTokens[2]).GetComparisonIndex();
	Result.SectionName = LocalSectionName.GetComparisonIndex();
	Result.ValueName = FMinimalName(LocalValueName);

	return Result;
}

void EscapeConfigTrackingTokenToString(FName Token, FStringBuilderBase& Result)
{
	Result.Reset();
	EscapeConfigTrackingTokenAppendString(Token, Result);
}

void EscapeConfigTrackingTokenAppendString(FName Token, FStringBuilderBase& Result)
{
	int32 InitialLength = Result.Len();
	Result << Token;
	FStringView AddedView = Result.ToView().RightChop(InitialLength);
	if (AddedView.Contains(TEXTVIEW(":")))
	{
		FString ReplaceText(AddedView);
		ReplaceText.ReplaceInline(TEXT(":"), TEXT("::"));
		Result.RemoveSuffix(AddedView.Len());
		Result << ReplaceText;
	}
}

bool TryTokenizeConfigTrackingString(FStringView Text, TArrayView<FStringBuilderBase*> OutTokens)
{
	int32 TextLen = Text.Len();
	if (TextLen == 0)
	{
		for (FStringBuilderBase* Token : OutTokens) Token->Reset();
		return false;
	}
	const TCHAR* TextData = Text.GetData();

	int32 NumTokens = OutTokens.Num();
	check(NumTokens > 0);
	int32 NextTokenIndex = 0;
	FStringBuilderBase* OutToken = OutTokens[NextTokenIndex++];
	OutToken->Reset();
	for (int32 Index = 0; Index < TextLen; ++Index)
	{
		TCHAR C = TextData[Index];
		if (C != ':')
		{
			OutToken->AppendChar(C);
		}
		else if (Index < TextLen - 1 && TextData[Index + 1] == ':')
		{
			++Index;
			OutToken->AppendChar(':');
		}
		else
		{
			if (OutToken->Len() == 0)
			{
				// An empty token and therefore the string is invalid; abandon anything that follows it
				while (NextTokenIndex < NumTokens) OutTokens[NextTokenIndex++]->Reset();
				return false;
			}
			if (NextTokenIndex >= NumTokens)
			{
				// Too many tokens
				return false;
			}
			OutToken = OutTokens[NextTokenIndex++];
			OutToken->Reset();
		}
	}
	if (OutToken->Len() == 0)
	{
		// Empty token
		while (NextTokenIndex < NumTokens) OutTokens[NextTokenIndex++]->Reset();
		return false;
	}
	if (NextTokenIndex < NumTokens)
	{
		// too few tokens
		while (NextTokenIndex < NumTokens) OutTokens[NextTokenIndex++]->Reset();
		return false;
	}
	return true;
}

} // namespace UE::ConfigAccessTracking

const TCHAR* LexToString(UE::ConfigAccessTracking::ELoadType LoadType)
{
	using namespace UE::ConfigAccessTracking;

	switch (LoadType)
	{
	case ELoadType::ConfigSystem: return TEXT("ConfigSystem");
	case ELoadType::LocalIniFile: return TEXT("LocalIniFile");
	case ELoadType::LocalSingleIniFile: return TEXT("LocalSingleIniFile");
	case ELoadType::ExternalIniFile: return TEXT("ExternalIniFile");
	case ELoadType::ExternalSingleIniFile: return TEXT("ExternalSingleIniFile");
	case ELoadType::Manual: return TEXT("Manual");
	case ELoadType::SuppressReporting: return TEXT("SuppressReporting");
	case ELoadType::Uninitialized:
	default:
		return TEXT("Uninitialized");
	}
}

void LexFromString(UE::ConfigAccessTracking::ELoadType& OutLoadType, FStringView Text)
{
	using namespace UE::ConfigAccessTracking;

	if (Text.IsEmpty())
	{
		OutLoadType = ELoadType::Uninitialized;
		return;
	}
	switch (Text[0])
	{
	case 'C':
	case 'c':
		if (Text.Equals(TEXT("ConfigSystem"), ESearchCase::IgnoreCase))
		{
			OutLoadType = ELoadType::ConfigSystem;
			return;
		}
		break;
	case 'L':
	case 'l':
		if (Text.Equals(TEXT("LocalIniFile"), ESearchCase::IgnoreCase))
		{
			OutLoadType = ELoadType::LocalIniFile;
			return;
		}
		if (Text.Equals(TEXT("LocalSingleIniFile"), ESearchCase::IgnoreCase))
		{
			OutLoadType = ELoadType::LocalSingleIniFile;
			return;
		}
		break;
	case 'E':
	case 'e':
		if (Text.Equals(TEXT("ExternalIniFile"), ESearchCase::IgnoreCase))
		{
			OutLoadType = ELoadType::ExternalIniFile;
			return;
		}
		if (Text.Equals(TEXT("ExternalSingleIniFile"), ESearchCase::IgnoreCase))
		{
			OutLoadType = ELoadType::ExternalSingleIniFile;
			return;
		}
		break;
	case 'S':
	case 's':
		if (Text.Equals(TEXT("SuppressReporting"), ESearchCase::IgnoreCase))
		{
			OutLoadType = ELoadType::SuppressReporting;
			return;
		}
		break;
	case 'M':
	case 'm':
		if (Text.Equals(TEXT("Manual"), ESearchCase::IgnoreCase))
		{
			OutLoadType = ELoadType::Manual;
			return;
		}
		break;
	default:
		break;
	}
	OutLoadType = ELoadType::Uninitialized;
}

#endif // UE_WITH_CONFIG_TRACKING || WITH_EDITOR
