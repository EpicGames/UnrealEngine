// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/ConfigAccessTracking.h"

#if UE_WITH_CONFIG_TRACKING
#include "AutoRTFM.h"
#include "Containers/Array.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/DataDrivenPlatformInfoRegistry.h"
#include "Templates/Tuple.h"

// LightWeightRWLock
#include <atomic>
#include "HAL/Platform.h"
#include "HAL/PlatformProcess.h"
#endif

#if UE_WITH_CONFIG_TRACKING

namespace UE::ConfigAccessTracking
{

thread_local bool bIgnoreReads = false;

FFile::FFile(const FConfigFile* InConfigFile)
	: ConfigFile(InConfigFile)
	, bSavedHasConfigFile(false)
	, bSavedConfigFileHasPlatformName(false)
	, bPlatformNameInitialized(false)
	, bOverridePlatformName(false)
	, bSavedOverridePlatformName(false)
{
}

FName FFile::GetFilenameToLoad() const
{
	if (!ConfigFile)
	{
		return NAME_None;
	}
	if (!OverrideFilenameToLoad.IsNone())
	{
		return OverrideFilenameToLoad;
	}
	return ConfigFile->Name;
}

FName FFile::GetPlatformName()
{
	bool bDirty = false;
	bool bHasConfigFile = ConfigFile != nullptr;
	if (!bPlatformNameInitialized)
	{
		bDirty = true;
	}
	else if (bSavedHasConfigFile != bHasConfigFile || bSavedOverridePlatformName != bOverridePlatformName)
	{
		bDirty = true;
	}
	// if !bHasConfigFile there is nothing further to test; not dirty
	else if (bHasConfigFile)
	{
		if (ConfigFile->bHasPlatformName != bSavedConfigFileHasPlatformName)
		{
			bDirty = true;
		}
		else
		{
			if (!ConfigFile->bHasPlatformName || ConfigFile->PlatformName.IsEmpty())
			{
				bDirty = !SavedConfigFilePlatformName.IsEmpty();
			}
			else
			{
				// Compare by pointer rather than by string compare, to be cheaper
				bDirty = (SavedConfigFilePlatformName.GetData() != *ConfigFile->PlatformName ||
					SavedConfigFilePlatformName.Len() != ConfigFile->PlatformName.Len());
			}
		}
	}
	if (!bDirty)
	{
		return PlatformName;
	}
	bSavedHasConfigFile = bHasConfigFile;
	bSavedOverridePlatformName = bOverridePlatformName;
	if (!bHasConfigFile)
	{
		bSavedConfigFileHasPlatformName = false;
		SavedConfigFilePlatformName.Reset();
	}
	else
	{
		bSavedConfigFileHasPlatformName = ConfigFile->bHasPlatformName;
		SavedConfigFilePlatformName = ConfigFile->PlatformName;
	}
	bPlatformNameInitialized = true;

	if (bOverridePlatformName)
	{
		// Currently the only time we need to override the platformname is when we are clearing it, so we don't keep a
		// separate variable for OverriddenPlatformName, we just interpret it as being NAME_None
		PlatformName = NAME_None;
	}
	else if (!ConfigFile)
	{
		PlatformName = NAME_None;
	}
	else if (ConfigFile->bHasPlatformName)
	{
		// The platform that was passed to LoadExternalIniFile
		PlatformName = FName(FStringView(ConfigFile->PlatformName));
	}
	else if (!GConfig || !GConfig->IsReadyForUse())
	{
		// All unmarked-platform config files read during config startup are platform-agnostic.
		// That property is required because we cannot fall back to the search through
		// GetAllPlatformInfos and SourceIniHierarchy below during startup, because they are not
		// yet threadsafe to access.
		PlatformName = NAME_None;
	}
	else if (ConfigFile->Branch != nullptr)
	{
		PlatformName = ConfigFile->Branch->Platform;
	}
	else
	{
		// if it didn't have a branch or a platformname in itself, there's no Platform to be found
		PlatformName = NAME_None;
	}
	return PlatformName;
}

void FFile::SetAsLoadTypeConfigSystem(FConfigCacheIni& ConfigSystem, FConfigFile& InConfigFile)
{
	check(&InConfigFile == this->ConfigFile);
	if (!ConfigSystem.IsGloballyRegistered())
	{
		// We only know how to load globally registered ConfigSystems. If this system is not globally registered leave its
		// FConfigFiles as Uninitialized LoadType.
		return;
	}
	InConfigFile.LoadType = UE::ConfigAccessTracking::ELoadType::ConfigSystem;
	if (&ConfigSystem == GConfig)
	{
		// GConfig's platform is set equal to the editor's platform (e.g. Windows), but we need to mark
		// FConfigFiles as coming from GConfig, so set the override platformname
		bOverridePlatformName = true;
	}
}

FSection::FSection(FFile& InFileAccess, FStringView InSectionName)
	: FileAccess(&InFileAccess)
	, SectionName(FName(InSectionName, NAME_NO_NUMBER_INTERNAL).GetComparisonIndex())
{
}

FIgnoreScope::FIgnoreScope()
	: bPreviousIgnoreReads(bIgnoreReads)
{
	bIgnoreReads = true;
}

FIgnoreScope::~FIgnoreScope()
{
	bIgnoreReads = bPreviousIgnoreReads;
}

namespace Private
{


// Simple lock class used, copied from ObjectHandleTracking.cpp. TODO: Replace with ParkingLot's lightweight RWLock
struct FLightweightRWLock
{
	static const int32 Unlocked = 0;
	static const int32 LockedOffset = 0x7fffffff;		// Subtracted for write lock, producing a large negative number

	std::atomic_int32_t Mutex = 0;
};

class FLightweightReadScopeLock
{
public:
	FLightweightReadScopeLock(FLightweightRWLock& InLock)
		: Lock(InLock)
	{
		int32 LockValue = Lock.Mutex.fetch_add(1, std::memory_order_acquire);
		while (LockValue < 0)
		{
			FPlatformProcess::Yield();
			LockValue = Lock.Mutex.load(std::memory_order_acquire);
		}
	}
	~FLightweightReadScopeLock()
	{
		Lock.Mutex.fetch_sub(1, std::memory_order_release);
	}

private:
	FLightweightRWLock& Lock;
};

class FLightweightWriteScopeLock
{
public:
	FLightweightWriteScopeLock(FLightweightRWLock& InLock)
		: Lock(InLock)
	{
		int32 ExpectedUnlocked = FLightweightRWLock::Unlocked;
		while (!InLock.Mutex.compare_exchange_weak(ExpectedUnlocked, -FLightweightRWLock::LockedOffset, std::memory_order_acquire))
		{
			FPlatformProcess::Yield();
			ExpectedUnlocked = FLightweightRWLock::Unlocked;
		}
	}
	~FLightweightWriteScopeLock()
	{
		Lock.Mutex.fetch_add(FLightweightRWLock::LockedOffset, std::memory_order_release);
	}
private:
	FLightweightRWLock& Lock;
};

std::atomic<int32> ConfigValueReadCallbackQuantity;

struct FConfigReadCallbacks
{
	static FConfigReadCallbacks& Get()
	{
		static FConfigReadCallbacks Callbacks;
		return Callbacks;
	}

	void OnConfigValueRead(UE::ConfigAccessTracking::FSection* Section, FMinimalName ValueName,
		const FConfigValue& ConfigValue)
	{
		FLightweightReadScopeLock _(HandleLock);
		for (auto& Pair : ConfigValueReadCallbacks)
		{
			Pair.Value(Section, ValueName, ConfigValue);
		}
	}

	FConfigValueReadCallbackId AddConfigValueReadCallback(FConfigValueReadCallbackFunc Func)
	{
		FConfigValueReadCallbackId Result;
		FLightweightWriteScopeLock _(HandleLock);
		NextHandleId++;
		ConfigValueReadCallbacks.Add({ NextHandleId, Func });
		++ConfigValueReadCallbackQuantity;
		Result = FConfigValueReadCallbackId{ NextHandleId };
		return Result;
	}

	void RemoveConfigValueReadCallback(FConfigValueReadCallbackId Handle)
	{
		FLightweightWriteScopeLock _(HandleLock);
		for (int32 i = ConfigValueReadCallbacks.Num() - 1; i >= 0; --i)
		{
			auto& Pair = ConfigValueReadCallbacks[i];
			if (Pair.Key == Handle.Id)
			{
				ConfigValueReadCallbacks.RemoveAt(i);
				--ConfigValueReadCallbackQuantity;
			}
		}
	}

private:
	int32 NextHandleId = 0;
	TArray<TTuple<int32, FConfigValueReadCallbackFunc>> ConfigValueReadCallbacks;
	FLightweightRWLock HandleLock;
};

void OnConfigValueReadInternal(UE::ConfigAccessTracking::FSection* Section, FMinimalName ValueName,
	const FConfigValue& ConfigValue)
{
	// By contract with FConfigFile::SuppressReporting we guarantee that we do not report reads of FConfigValue of
	// suppressed ConfigFiles; we implement this by early exiting if the ConfigFile pointer is null.
	// By contract with AddConfigValueReadCallback, we additionally guarantee that ConfigFile pointer is available in
	// the reported information.
	if (!Section || !Section->FileAccess || !Section->FileAccess->ConfigFile)
	{
		return;
	}

	// Implementation of FIgnoreScope
	if (bIgnoreReads)
	{
		return;
	}

	FConfigReadCallbacks::Get().OnConfigValueRead(Section, ValueName, ConfigValue);
}

} // namespace Private

FConfigValueReadCallbackId AddConfigValueReadCallback(FConfigValueReadCallbackFunc Callback)
{
	return UE::ConfigAccessTracking::Private::FConfigReadCallbacks::Get().AddConfigValueReadCallback(Callback);
}

void RemoveConfigValueReadCallback(FConfigValueReadCallbackId Handle)
{
	UE::ConfigAccessTracking::Private::FConfigReadCallbacks::Get().RemoveConfigValueReadCallback(Handle);
}

} // namespace UE::ConfigAccessTracking

#endif
