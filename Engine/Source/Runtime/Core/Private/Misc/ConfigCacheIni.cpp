// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/ConfigCacheIni.h"
#include "Misc/ConfigContext.h"
#include "Misc/ConfigHierarchy.h"
#include "Misc/ConfigUtilities.h"
#include "Containers/StringView.h"
#include "Misc/DateTime.h"
#include "Misc/MessageDialog.h"
#include "HAL/FileManager.h"
#include "Misc/FileHelper.h"
#include "Misc/CommandLine.h"
#include "Math/Vector4.h"
#include "Stats/Stats.h"
#include "HAL/IConsoleManager.h"
#include "HAL/LowLevelMemStats.h"
#include "HAL/LowLevelMemTracker.h"
#include "HAL/PlatformTLS.h"
#include "HAL/ThreadManager.h"
#include "Misc/CoreDelegates.h"
#include "Misc/App.h"
#include "Misc/RemoteConfigIni.h"
#include "Misc/DefaultValueHelper.h"
#include "Misc/ConfigManifest.h"
#include "Misc/DataDrivenPlatformInfoRegistry.h"
#include "Misc/StringBuilder.h"
#include "Misc/PathViews.h"
#include "Misc/ScopeLock.h"
#include "Misc/ScopeRWLock.h"
#include "Misc/TransactionallySafeRWLock.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "ProfilingDebugging/AssetMetadataTrace.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/LargeMemoryReader.h"
#include "Async/Async.h"
#include "Async/ParallelFor.h"
#include "Async/SharedLock.h"
#include "Async/UniqueLock.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Logging/MessageLog.h"
#include <limits>

// if set to 1, this will make sure FConfigCacheIni::GetString (and friends) is thread safe from dynamic layer loading, and SafeReload
// if set to 0, this will behave as 2025 and before (no safety for GetString)
// note that if you bypass FConfigCache::Get* and use FConfigFile::Get*, then it will not be threadsafe
#define UE_THREAD_SAFETY_PROTECT_GETSTRING 1

// if set to 1, GetString (GetInt, etc) will be slower, but SafeUnload will unload more memory
// if set to 0, SafeUnload will free less memory, but Get* will be faster
#define UE_THREAD_SAFETY_USE_WRITE_LOCKS 1

namespace
{
	const FString CurrentIniVersionStr = "CurrentIniVersion";
	const FName VersionSectionName = "Version";
	const FString SectionsToSaveStr = "SectionsToSave";

	TMap<FString, FString> SectionRemap;
	TMap<FString, TMap<FString, FString>> KeyRemap;

	TMap<TCHAR, FConfigValue::EValueType> CommandLookup = {
		{ '\0', FConfigValue::EValueType::Set },
		{ '-', FConfigValue::EValueType::Remove },
		{ '+', FConfigValue::EValueType::ArrayAddUnique },
		{ '.', FConfigValue::EValueType::ArrayAdd },
		{ '!', FConfigValue::EValueType::Clear },
		{ '@', FConfigValue::EValueType::ArrayOfStructKey },
		{ '*', FConfigValue::EValueType::POCArrayOfStructKey },
		{ '^', FConfigValue::EValueType::InitializeToEmpty },
	};
}

DEFINE_LOG_CATEGORY(LogConfig);
#define LOCTEXT_NAMESPACE "ConfigCache"


static TAutoConsoleVariable<int32> CVarUseNewDynamicLayers(
	TEXT("ini.UseNewDynamicLayers"),
	1,
	TEXT("If true, use the new dynamic layers that load/unload, with GameFeatures and Hotfixes"),
	ECVF_Default);

static int GUseNewSaveTracking = 0;
static FAutoConsoleVariableRef CVarUseNewSaveTracking(
	TEXT("ini.UseNewSaveTracking"),
	GUseNewSaveTracking,
	TEXT("If true, use the new method for tracking modifications to GConfig when saving"));

static int GTimeToUnloadConfig = 0;
static FAutoConsoleVariableRef CVarTimeToUnloadConfig(
	TEXT("ini.TimeToUnloadConfig"),
	GTimeToUnloadConfig,
	TEXT("If > 0, when a config branch hasn't been accessed in this many seconds, SafeUnload the branch"));

static FString GConfigBranchesToNeverUnload;
static FAutoConsoleVariableRef CVarConfigBranchesToNeverUnload(
	TEXT("ini.ConfigBranchesToNeverUnload"),
	GConfigBranchesToNeverUnload,
	TEXT("A comma separated list of config branch names that should never be unloaded."));

static int GSkipSaveIfPreSaveSucceeded = 1;
static FAutoConsoleVariableRef CVarSkipSaveIfPreSaveSucceeded(
	TEXT("ini.SkipSaveIfPreSaveSucceeded"),
	GSkipSaveIfPreSaveSucceeded,
	TEXT("If true, then the standard config file save will be skipped if the PreSave delegate indicates someone saved."));


#if WITH_EDITOR
// editor wants full replay - and we can't put this in a cvar since we need way too early for cvar processing!
static int GDefaultReplayMethod = 2;
#else
static int GDefaultReplayMethod = 1;
#endif

static bool OverrideFileFromCommandline(FString & InOutFilename);

/**
 * Stores some values that are helpful for logging but would be cumbersome to pass around to every internal function.
 */
struct FConfigLogContext
{
	FConfigLogContext* OuterContext = nullptr;
	int32 ContextDepth = 0;

	const TCHAR* Action = nullptr;
	const TCHAR* Section = nullptr;
	const TCHAR* Key = nullptr;
};

thread_local static FConfigLogContext ConfigLogContext;

/**
 * Pushes a new set of logging context values for the current scope.
 */
struct FConfigLogContextScope
{
	FConfigLogContext PrevLogContext;

	FConfigLogContextScope(const TCHAR* InAction, const TCHAR* InSection, const TCHAR* InKey)
	{
		PrevLogContext = ConfigLogContext;

		// Only set the outer pointer if this isn't the first scope
		if (ConfigLogContext.ContextDepth > 0)
		{
			ConfigLogContext.OuterContext = &PrevLogContext;
		}

		ConfigLogContext.ContextDepth++;
		ConfigLogContext.Action = InAction;
		ConfigLogContext.Section = InSection;
		ConfigLogContext.Key = InKey;
	}

	~FConfigLogContextScope()
	{
		// Verify OuterContext pointer was not modified
		check((ConfigLogContext.OuterContext == nullptr) || (ConfigLogContext.OuterContext == &PrevLogContext));

		ConfigLogContext = PrevLogContext;
	}
};

/**
 * Gets the "outermost" log context. Useful for determining which function was called from outside code.
 * e.g. FindConfigFile can be called directly, but is also called by Find and FindPlatformConfig.
 */
FConfigLogContext* GetOutermostConfigLogContext()
{
	FConfigLogContext* Result = &ConfigLogContext;
	while (Result->OuterContext)
	{
		Result = Result->OuterContext;
	}

	return Result;
}

/*-----------------------------------------------------------------------------
FConfigValue
-----------------------------------------------------------------------------*/

struct FConfigExpansion
{
	template<int N>
	FConfigExpansion(const TCHAR(&Var)[N], FString&& Val)
		: Variable(Var)
		, Value(MoveTemp(Val))
		, VariableLen(N - 1)
	{}

	template<int N>
	FConfigExpansion(const TCHAR(&Var)[N], const FString& Val)
		: Variable(Var)
		, Value(Val)
		, VariableLen(N - 1)
	{}

	const TCHAR* Variable;
	FString Value;
	int VariableLen;
};

static FString GetApplicationSettingsDirNormalized()
{
	FString Dir = FPlatformProcess::ApplicationSettingsDir();
	FPaths::NormalizeFilename(Dir);
	return Dir;
}

static const FConfigExpansion* MatchExpansions(const TCHAR* PotentialVariable)
{
	// Allocate replacement value strings once
	static const FConfigExpansion Expansions[] =
	{
		FConfigExpansion(TEXT("%GAME%"), FString(FApp::GetProjectName())),
		FConfigExpansion(TEXT("%GAMEDIR%"), FPaths::ProjectDir()),
		FConfigExpansion(TEXT("%ENGINEDIR%"), FPaths::EngineDir()),
		FConfigExpansion(TEXT("%ENGINEUSERDIR%"), FPaths::EngineUserDir()),
		FConfigExpansion(TEXT("%ENGINEVERSIONAGNOSTICUSERDIR%"), FPaths::EngineVersionAgnosticUserDir()),
		FConfigExpansion(TEXT("%APPSETTINGSDIR%"), GetApplicationSettingsDirNormalized()),
		FConfigExpansion(TEXT("%GAMESAVEDDIR%"), FPaths::ProjectSavedDir()),
	};

	for (const FConfigExpansion& Expansion : Expansions)
	{
		if (FCString::Strncmp(Expansion.Variable, PotentialVariable, Expansion.VariableLen) == 0)
		{
			return &Expansion;
		}
	}

	return nullptr;
}

static const FConfigExpansion* FindNextExpansion(const TCHAR* Str, const TCHAR*& OutMatch)
{
	for (const TCHAR* It = FCString::Strchr(Str, '%'); It; It = FCString::Strchr(It + 1, '%'))
	{
		if (const FConfigExpansion* Expansion = MatchExpansions(It))
		{
			OutMatch = It;
			return Expansion;
		}
	}

	return nullptr;
}

bool FConfigValue::ExpandValue(const FString& InCollapsedValue, FString& OutExpandedValue)
{
	struct FSubstring
	{
		const TCHAR* Begin;
		const TCHAR* End;

		int32 Len() const { return UE_PTRDIFF_TO_INT32(End - Begin); }
	};

	// Find substrings of input and expansions to concatenate to final output string
	TArray<FSubstring, TFixedAllocator<7>> Substrings;
	const TCHAR* It = *InCollapsedValue;
	while (true)
	{
		const TCHAR* Match;
		if (const FConfigExpansion* Expansion = FindNextExpansion(It, Match))
		{
			Substrings.Add({ It, Match });
			Substrings.Add({ *Expansion->Value, (*Expansion->Value) + Expansion->Value.Len() });

			It = Match + Expansion->VariableLen;
		}
		else if (Substrings.Num() == 0)
		{
			// No expansions matched, skip concatenation and return input string
			OutExpandedValue = InCollapsedValue;
			return false;
		}
		else
		{
			Substrings.Add({ It, *InCollapsedValue + InCollapsedValue.Len() });
			break;
		}
	}

	// Concat
	int32 OutLen = 0;
	for (const FSubstring& Substring : Substrings)
	{
		OutLen += Substring.Len();
	}

	OutExpandedValue.Reserve(OutLen);
	for (const FSubstring& Substring : Substrings)
	{
		OutExpandedValue.AppendChars(Substring.Begin, Substring.Len());
	}

	return true;
}

FString FConfigValue::ExpandValue(const FString& InCollapsedValue)
{
	FString OutExpandedValue;
	ExpandValue(InCollapsedValue, OutExpandedValue);
	return OutExpandedValue;
}

bool FConfigValue::NeedsToExpandValue()
{
	const TCHAR* Dummy;
	return FindNextExpansion(*SavedValue, Dummy) != nullptr;
}

bool FConfigValue::CollapseValue(const FString& InExpandedValue, FString& OutCollapsedValue)
{
	int32 NumReplacements = 0;
	OutCollapsedValue = InExpandedValue;

	auto ExpandPathValueInline = [&](const FString& InPath, const TCHAR* InReplacement)
	{
		if (OutCollapsedValue.StartsWith(InPath, ESearchCase::CaseSensitive))
		{
			NumReplacements += OutCollapsedValue.ReplaceInline(*InPath, InReplacement, ESearchCase::CaseSensitive);
		}
		else if (FPaths::IsRelative(InPath))
		{
			const FString AbsolutePath = FPaths::ConvertRelativePathToFull(InPath);
			if (OutCollapsedValue.StartsWith(AbsolutePath, ESearchCase::CaseSensitive))
			{
				NumReplacements += OutCollapsedValue.ReplaceInline(*AbsolutePath, InReplacement, ESearchCase::CaseSensitive);
			}
		}
	};

	// Replace the game directory with %GAMEDIR%.
	ExpandPathValueInline(FPaths::ProjectDir(), TEXT("%GAMEDIR%"));

	// Replace the user's engine directory with %ENGINEUSERDIR%.
	ExpandPathValueInline(FPaths::EngineUserDir(), TEXT("%ENGINEUSERDIR%"));

	// Replace the user's engine agnostic directory with %ENGINEVERSIONAGNOSTICUSERDIR%.
	ExpandPathValueInline(FPaths::EngineVersionAgnosticUserDir(), TEXT("%ENGINEVERSIONAGNOSTICUSERDIR%"));

	// Replace the application settings directory with %APPSETTINGSDIR%.
	FString AppSettingsDir = FPlatformProcess::ApplicationSettingsDir();
	FPaths::NormalizeFilename(AppSettingsDir);
	ExpandPathValueInline(AppSettingsDir, TEXT("%APPSETTINGSDIR%"));

	// Note: We deliberately don't replace the game name with %GAME% here, as the game name may exist in many places (including paths)

	return NumReplacements > 0;
}

FString FConfigValue::CollapseValue(const FString& InExpandedValue)
{
	FString CollapsedValue;
	CollapseValue(InExpandedValue, CollapsedValue);
	return CollapsedValue;
}

#if !UE_BUILD_SHIPPING
/**
* Checks if the section name is the expected name format (long package name or simple name)
*/
static void CheckLongSectionNames(const TCHAR* Section, const FConfigFile* File)
{
	if (!FPlatformProperties::RequiresCookedData())
	{
		// Guard against short names in ini files.
		if (FCString::Strnicmp(Section, TEXT("/Script/"), 8) == 0)
		{
			// Section is a long name
			if (File->FindSection(Section + 8))
			{
				UE_LOGF(LogConfig, Fatal, "Short config section found while looking for %ls", Section);
			}
		}
		else
		{
			// Section is a short name
			FString LongName = FString("/Script/") + Section;
			if (File->FindSection(*LongName))
			{
				UE_LOGF(LogConfig, Fatal, "Short config section used instead of long %ls", Section);
			}
		}
	}
}
#endif // UE_BUILD_SHIPPING




/**
 * Check if an ini file exists, allowing a delegate to determine if it will handle loading it
 */
bool DoesConfigFileExistWrapper(const TCHAR* IniFile, const TSet<FString>* IniCacheSet, const TSet<FString>* PrimaryConfigFileCache, const TSet<FString>* SecondaryConfigFileCache)
{
	// will any delegates return contents via TSPreLoadConfigFileDelegate()?
	int32 ResponderCount = 0;
	FCoreDelegates::TSCountPreLoadConfigFileRespondersDelegate().Broadcast(IniFile, ResponderCount);

	if (ResponderCount > 0)
	{
		return true;
	}

	FString IniFileString = IniFile;
	if (OverrideFileFromCommandline(IniFileString))
	{
		return true;
	}

	// check staged cache (likely for plugin configs)
	if (PrimaryConfigFileCache != nullptr || SecondaryConfigFileCache != nullptr)
	{
		return (PrimaryConfigFileCache && PrimaryConfigFileCache->Contains(IniFileString)) ||
			(SecondaryConfigFileCache && SecondaryConfigFileCache->Contains(IniFileString));
	}

	// So far, testing on cooked consoles, cooked desktop, and the Editor
	// works fine.
	// However, there was an issue where INIs wouldn't be found during cooking,
	// which would pass by silently. Realistically, in most cases this would never
	// have caused an issue, but using FPlatformProperties::RequiresCookedData
	// to prevent using the cache in that case ensures full consistency.
	if (IniCacheSet && FPlatformProperties::RequiresCookedData())
	{
		const bool bFileExistsCached = IniCacheSet->Contains(IniFileString);

		// This code can be uncommented if we expect there are INIs that are not being
		// found in the cache.
		/**
		const bool bFileExistsCachedTest = IFileManager::Get().FileSize(IniFile) >= 0;
		ensureMsgf(
			bFileExistsCached == bFileExistsCachedTest,
			TEXT("DoesConfigFileExistWrapper: InCache = %d, InFileSystem = %d, Name = %s, Configs = \n%s"),
			!!bFileExistsCached,
			!!bFileExistsCachedTest,
			IniFile,
			*FString::Join(*IniCacheSet, TEXT("\n"))
		);
		*/

		return bFileExistsCached;
	}

	// otherwise just look for the normal file to exist
	UE_LOGF(LogConfig, VeryVerbose, "Looking for a config file without a staged cache for %ls", IniFile);
	const bool bFileExistsCached = IFileManager::Get().FileExists(IniFile);
	return bFileExistsCached;
}

/**
 * Load ini file, but allowing a delegate to handle the loading instead of the standard file load
 */
static bool LoadConfigFileWrapper(const TCHAR* IniFile, FString& Contents, bool bIsOverride = false)
{
	// We read the Base.ini and PluginBase.ini files many many times, so cache them
	static FString BaseIniContents;
	static FString PluginBaseIniContents;

	const TCHAR* LastSlash = FCString::Strrchr(IniFile, '/');
	if (LastSlash == nullptr)
	{
		LastSlash = FCString::Strrchr(IniFile, '\\');
	}

	bool bIsBaseIni = LastSlash != nullptr && FCString::Stricmp(LastSlash + 1, TEXT("Base.ini")) == 0;
	if (bIsBaseIni && BaseIniContents.Len() > 0)
	{
		Contents = BaseIniContents;
		return true;
	}

	bool bIsPluginBaseIni = LastSlash != nullptr && FCString::Stricmp(LastSlash + 1, TEXT("PluginBase.ini")) == 0;
	if (bIsPluginBaseIni && PluginBaseIniContents.Len() > 0)
	{
		Contents = PluginBaseIniContents;
		return true;
	}

	// let other systems load the file instead of the standard load below
	FCoreDelegates::TSPreLoadConfigFileDelegate().Broadcast(IniFile, Contents);

	// if this loaded any text, we are done, and we won't override the contents with standard ini file data
	if (Contents.Len())
	{
		return true;
	}

#if ALLOW_INI_OVERRIDE_FROM_COMMANDLINE
	if (bIsOverride)
	{
		// Make sure we bypass the pak layer because if our override is likely under root the pak layer will
		// just resolve it even if it's an absolute path.

		return FFileHelper::LoadFileToString(Contents, &IPlatformFile::GetPlatformPhysical(), IniFile);
	}
#endif

	// note: we don't check if FileOperations are disabled because downloadable content calls this directly (which
	// needs file ops), and the other caller of this is already checking for disabled file ops
	// and don't read from the file, if the delegate got anything loaded
	bool bResult = FFileHelper::LoadFileToString(Contents, IniFile);
	if (bResult)
	{
		if (bIsBaseIni)
		{
			BaseIniContents = Contents;
		}
		else if (bIsPluginBaseIni)
		{
			PluginBaseIniContents = Contents;
		}
	}
	return bResult;
}

/**
 * Save an ini file, with delegates also saving the file (its safe to allow both to happen, even tho loading doesn't behave this way)
 */
static bool SaveConfigFileWrapper(const TCHAR* IniFile, const FString& Contents)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SaveConfigFileWrapper);

	// let anyone that needs to save it, do so (counting how many did)
	int32 SavedCount = 0;
	FCoreDelegates::TSPreSaveConfigFileDelegate().Broadcast(IniFile, Contents, SavedCount);

	// save if the delegate didn't, or it did and we don't want to skip the normal save
	bool bLocalWriteSucceeded = false;
	if (SavedCount == 0 || GSkipSaveIfPreSaveSucceeded == false)
	{
		if (FConfigFile::WriteTempFileThenMove())
		{
			const FString BaseFilename = FPaths::GetBaseFilename(IniFile);
			const FString TempFilename = FPaths::CreateTempFilename(*FPaths::ProjectSavedDir(), *BaseFilename.Left(32));
			bLocalWriteSucceeded = FFileHelper::SaveStringToFile(Contents, *TempFilename, FFileHelper::EEncodingOptions::ForceUTF8WithoutBOM);
			if (bLocalWriteSucceeded)
			{
				if (!IFileManager::Get().Move(IniFile, *TempFilename))
				{
					IFileManager::Get().Delete(*TempFilename);
					bLocalWriteSucceeded = false;
				}
			}
		}
		else
		{
			bLocalWriteSucceeded = FFileHelper::SaveStringToFile(Contents, IniFile, FFileHelper::EEncodingOptions::ForceUTF8WithoutBOM);
		}
	}

	// success is based on a delegate or file write working (or both)
	return SavedCount > 0 || bLocalWriteSucceeded;
}

static bool DeleteConfigFileWrapper(const TCHAR* IniFile)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(DeleteConfigFileWrapper);

	bool bDeleted = false;
	FCoreDelegates::TSPreDeleteConfigFileDelegate().Broadcast(IniFile, bDeleted);

	bDeleted |= IFileManager::Get().Delete(IniFile);
	return bDeleted;
}



/**
 * Functionality to assist with updating a config file with one property value change.
 */
class FSinglePropertyConfigHelper
{
public:
	/**
	 * We need certain information for the helper to be useful.
	 *
	 * @Param InIniFilename - The disk location of the file we wish to edit.
	 * @Param InSectionName - The section the property belongs to.
	 * @Param InPropertyName - The name of the property that has been edited.
	 * @Param InPropertyValue - The new value of the property that has been edited, or unset to remove the property.
	 */
	FSinglePropertyConfigHelper(const FString& InIniFilename, const FString& InSectionName, const FString& InPropertyName, const TOptional<FString>& InPropertyValue)
		: IniFilename(InIniFilename)
		, SectionName(InSectionName)
		, PropertyName(InPropertyName)
		, PropertyValue(InPropertyValue)
	{
		// Split the file into the necessary parts.
		PopulateFileContentHelper();
	}

	/**
	 * Setup this helper for writing a property to a static layer section (see ModifyAndWriteLayer)
	 *
	 * @Param InIniFilename - The disk location of the file we wish to edit.
	 * @Param InSectionName - The section the property belongs to.
	 * @Param InPropertyName - The name of the property that has been edited.
	 * @Param InLayerSection - The section of a static layer that contains the value(s) to write out (needed to write arrays correctly)
	 */
	FSinglePropertyConfigHelper(const FString& InIniFilename, const FString& InSectionName, const FString& InPropertyName, FConfigCommandStreamSection* InLayerSection)
		: IniFilename(InIniFilename)
		, SectionName(InSectionName)
		, PropertyName(InPropertyName)
		, bUpdateLayer(true)
		, LayerSection(InLayerSection)
	{
		// Split the file into the necessary parts.
		PopulateFileContentHelper();
	}

	/**
	 * Perform the action of updating the config file with the new property value.
	 */
	bool UpdateConfigFile()
	{
		UpdatePropertyInSection();
		// Rebuild the file with the updated section.

		FString NewFile = IniFileMakeup.BeforeSection + IniFileMakeup.Section + IniFileMakeup.AfterSection;
		if (!NewFile.EndsWith(TEXT(LINE_TERMINATOR_ANSI LINE_TERMINATOR_ANSI)))
		{
			NewFile.AppendChars(LINE_TERMINATOR, TCString<TCHAR>::Strlen(LINE_TERMINATOR));
		}
		return SaveConfigFileWrapper(*IniFilename, NewFile);
	}


private:

	/**
	 * Clear any trailing whitespace from the end of the output.
	 */
	void ClearTrailingWhitespace(FString& InStr)
	{
		const FString Endl(LINE_TERMINATOR);
		while (InStr.EndsWith(LINE_TERMINATOR, ESearchCase::CaseSensitive))
		{
			InStr.LeftChopInline(Endl.Len(), EAllowShrinking::No);
		}
	}

	/**
	 * Update the section with the new value for the property.
	 */
	void UpdatePropertyInSection()
	{
		FString UpdatedSection;
		if (IniFileMakeup.Section.IsEmpty())
		{
			if (PropertyValue.IsSet() || bUpdateLayer)
			{
				const FString DecoratedSectionName = FString::Printf(TEXT("[%s]"), *SectionName);

				ClearTrailingWhitespace(IniFileMakeup.BeforeSection);
				UpdatedSection += LINE_TERMINATOR;
				UpdatedSection += LINE_TERMINATOR;
				UpdatedSection += DecoratedSectionName;
				AppendPropertyLine(UpdatedSection);
			}
		}
		else
		{
			FString SectionLine;
			const TCHAR* Ptr = *IniFileMakeup.Section;
			bool bUpdatedPropertyOnPass = false;
			while (Ptr != nullptr && FParse::Line(&Ptr, SectionLine, true))
			{
				FString KeyAndEqual = PropertyName + TEXT("=");
				if (SectionLine.Len() > 0)
				{
					if (SectionLine.StartsWith(KeyAndEqual) ||
					(bUpdateLayer && CommandLookup.Contains(SectionLine[0]) && SectionLine.Mid(1).StartsWith(KeyAndEqual)))
					{
						if (!bUpdatedPropertyOnPass)
						{
							if (PropertyValue.IsSet() || bUpdateLayer)
							{
								AppendPropertyLine(UpdatedSection);
							}
						}
						bUpdatedPropertyOnPass = true;
					}
					else
					{
						UpdatedSection += SectionLine;
						UpdatedSection += LINE_TERMINATOR;
					}
				}
				else
				{
					UpdatedSection += LINE_TERMINATOR;
				}
			}

			// If the property wasnt found in the text of the existing section content,
			// append it to the end of the section.
			if (!bUpdatedPropertyOnPass && (bUpdateLayer || PropertyValue.IsSet()))
			{
				AppendPropertyLine(UpdatedSection);
			}
		}

		UpdatedSection += LINE_TERMINATOR;
		IniFileMakeup.Section = UpdatedSection;

	}

	/**
	 * Split the file up into parts:
	 * -> Before the section we wish to edit, which will remain unaltered,
	 * ->-> The section we wish to edit, we only seek to edit the single property,
	 * ->->-> After the section we wish to edit, which will remain unaltered.
	 */
	void PopulateFileContentHelper()
	{
		FString UnprocessedFileContents;
		if (LoadConfigFileWrapper(*IniFilename, UnprocessedFileContents))
		{
			// Find the section in the file text.
			const FString DecoratedSectionName = FString::Printf(TEXT("[%s]"), *SectionName);

			const int32 DecoratedSectionNameStartIndex = UnprocessedFileContents.Find(DecoratedSectionName);
			if (DecoratedSectionNameStartIndex != INDEX_NONE)
			{
				// If we found the section, cache off the file text before the section.
				IniFileMakeup.BeforeSection = UnprocessedFileContents.Left(DecoratedSectionNameStartIndex);
				UnprocessedFileContents.RemoveAt(0, IniFileMakeup.BeforeSection.Len());

				// For the rest of the file, split it into the section we are editing and the rest of the file after.
				const TCHAR* Ptr = UnprocessedFileContents.Len() > 0 ? *UnprocessedFileContents : nullptr;
				FString NextUnprocessedLine;
				bool bReachedNextSection = false;
				while (Ptr != nullptr && FParse::Line(&Ptr, NextUnprocessedLine, true))
				{
					bReachedNextSection |= (NextUnprocessedLine.StartsWith(TEXT("[")) && NextUnprocessedLine != DecoratedSectionName);
					if (bReachedNextSection)
					{
						IniFileMakeup.AfterSection += NextUnprocessedLine;
						IniFileMakeup.AfterSection += LINE_TERMINATOR;
					}
					else
					{
						IniFileMakeup.Section += NextUnprocessedLine;
						IniFileMakeup.Section += LINE_TERMINATOR;
					}
				}
			}
			else
			{
				IniFileMakeup.BeforeSection = UnprocessedFileContents;
			}
		}
	}

	/**
	 * Append the property entry to the section
	 */
	void AppendPropertyLine(FString& PreText)
	{
		check(PropertyValue.IsSet() || bUpdateLayer);

		// Make sure we dont leave much whitespace, and append the property name/value entry
		ClearTrailingWhitespace(PreText);
		PreText += LINE_TERMINATOR;

		if (bUpdateLayer)
		{
			// get every value matching the key, as an array will have multiple, or we could have a -Foo=1, +Foo=2
			TArray<FConfigValue*> Values;
			LayerSection->MultiFindPointer(*PropertyName, Values, true);
			for (const FConfigValue* Value : Values)
			{
				// export each one with a command/value
				if (Value->ValueType != FConfigValue::EValueType::Set)
				{
					const TCHAR* Cmd = CommandLookup.FindKey(Value->ValueType);
					PreText.AppendChar(*Cmd);
				}
				FConfigFile::AppendExportedPropertyLine(PreText, PropertyName, Value->GetSavedValueForWriting());
			}
		}
		else
		{
			PreText += FConfigFile::GenerateExportedPropertyLine(PropertyName, PropertyValue.GetValue());
		}
	}


private:
	// The disk location of the ini file we seek to edit
	FString IniFilename;

	// The section in the config file
	FString SectionName;

	// The name of the property that has been changed
	FString PropertyName;

	// Setting when updating value(s) in a layer FConfigCommandStream
	bool bUpdateLayer = false;
	FConfigCommandStreamSection* LayerSection;

	// The new value, in string format, of the property that has been changed
	// This will be unset if the property has been removed from the config
	TOptional<FString> PropertyValue;

	// Helper struct that holds the makeup of the ini file.
	struct IniFileContent
	{
		// The section we wish to edit
		FString Section;

		// The file contents before the section we are editing
		FString BeforeSection;

		// The file contents after the section we are editing
		FString AfterSection;
	} IniFileMakeup; // Instance of the helper to maintain file structure.
};



/*-----------------------------------------------------------------------------
	FConfigSection
-----------------------------------------------------------------------------*/


bool FConfigSection::HasQuotes( const FString& Test )
{
	if (Test.Len() < 2)
	{
		return false;
	}

	return Test.Left(1) == TEXT("\"") && Test.Right(1) == TEXT("\"");
}

bool FConfigSection::operator==( const FConfigSection& B ) const
{
	const FConfigSection&A = *this;
	if ( A.Pairs.Num() != B.Pairs.Num() )
	{
		return false;
	}

	FConfigSectionMap::TConstIterator AIter(A), BIter(B);
	while (AIter && BIter)
	{
		if (AIter.Key() != BIter.Key())
		{
			return false;
		}

		const FString& AIterValue = AIter.Value().GetValue();
		const FString& BIterValue = BIter.Value().GetValue();
		if ( FCString::Strcmp(*AIterValue,*BIterValue) &&
			(!HasQuotes(AIterValue) || FCString::Strcmp(*BIterValue,*AIterValue.Mid(1, AIterValue.Len() - 2))) &&
			(!HasQuotes(BIterValue) || FCString::Strcmp(*AIterValue,*BIterValue.Mid(1, BIterValue.Len() - 2))) )
		{
			return false;
		}

		++AIter, ++BIter;
	}
	return true;
}

bool FConfigSection::operator!=( const FConfigSection& Other ) const
{
	return ! (FConfigSection::operator==(Other));
}

bool FConfigSection::AreSectionsEqualForWriting(const FConfigSection& A, const FConfigSection& B)
{
	if (A.Pairs.Num() != B.Pairs.Num())
	{
		return false;
	}

	FConfigSectionMap::TConstIterator AIter(A), BIter(B);
	while (AIter && BIter)
	{
		if (AIter.Key() != BIter.Key())
		{
			return false;
		}

		const FString& AIterValue = AIter.Value().GetValueForWriting();
		const FString& BIterValue = BIter.Value().GetValueForWriting();
		if (FCString::Strcmp(*AIterValue, *BIterValue) &&
			(!FConfigSection::HasQuotes(AIterValue) || FCString::Strcmp(*BIterValue, *AIterValue.Mid(1, AIterValue.Len() - 2))) &&
			(!FConfigSection::HasQuotes(BIterValue) || FCString::Strcmp(*AIterValue, *BIterValue.Mid(1, BIterValue.Len() - 2))))
		{
			return false;
		}

		++AIter, ++BIter;
	}
	return true;
}

FArchive& operator<<(FArchive& Ar, FConfigSection& ConfigSection)
{
	Ar << static_cast<FConfigSection::Super&>(ConfigSection);
	Ar << ConfigSection.ArrayOfStructKeys;
	return Ar;
}

// Pull out a property from a Struct property, StructKeyMatch should be in the form "MyProp=". This reduces
// memory allocations for each attempted match
static void ExtractPropertyValue(const FString& FullStructValue, const FString& StructKeyMatch, FString& Out)
{
	Out.Reset();

	int32 MatchLoc = FullStructValue.Find(StructKeyMatch);
	// we only look for matching StructKeys if the incoming Value had a key
	if (MatchLoc >= 0)
	{
		// skip to after the match string
		MatchLoc += StructKeyMatch.Len();

		const TCHAR* Start = &FullStructValue.GetCharArray()[MatchLoc];
		bool bInQuotes = false;
		// skip over an open quote
		if (*Start == '\"')
		{
			Start++;
			bInQuotes = true;
		}
		const TCHAR* Travel = Start;

		// look for end of token, using " if it started with one
		while (*Travel && ((bInQuotes && *Travel != '\"') || (!bInQuotes && (FChar::IsAlnum(*Travel) || *Travel == '_'))))
		{
			Travel++;
		}

		// pull out the token
		Out.AppendChars(Start, UE_PTRDIFF_TO_INT32(Travel - Start));
	}
}

void FConfigSection::HandleAddCommand(FName ValueName, FString&& Value, bool bAppendValueIfNotArrayOfStructsKeyUsed)
{
	if (!HandleArrayOfKeyedStructsCommand(ValueName, Forward<FString&&>(Value)))
	{
		if (bAppendValueIfNotArrayOfStructsKeyUsed)
		{
			Add(ValueName, FConfigValue(this, ValueName, MoveTemp(Value), FConfigValue::EValueType::ArrayCombined));
		}
		else
		{
			AddUnique(ValueName, FConfigValue(this, ValueName, MoveTemp(Value), FConfigValue::EValueType::ArrayCombined));
		}
	}
}

bool FConfigSection::HandleArrayOfKeyedStructsCommand(FName Key, FString&& Value)
{
	FString* StructKey = ArrayOfStructKeys.Find(Key);
	bool bHandledWithKey = false;
	if (StructKey)
	{
		// look at the incoming value for the StructKey
		FString StructKeyMatch = *StructKey + "=";

		// pull out the token that matches the StructKey (a property name) from the full struct property string
		FString StructKeyValueToMatch;
		ExtractPropertyValue(Value, StructKeyMatch, StructKeyValueToMatch);

		if (StructKeyValueToMatch.Len() > 0)
		{
			FString ExistingStructValueKey;
			// if we have a key for this array, then we look for it in the Value for each array entry
			for (FConfigSection::TIterator It(*this); It; ++It)
			{
				// only look at matching keys
				if (It.Key() == Key)
				{
					// now look for the matching ArrayOfStruct Key as the incoming KeyValue
					{
						const FString& ItValue = It.Value().GetValueForWriting(); // Don't report to AccessTracking
						ExtractPropertyValue(ItValue, StructKeyMatch, ExistingStructValueKey);
					}
					if (ExistingStructValueKey == StructKeyValueToMatch)
					{
						// we matched the key, so replace the existing value in place (so as not to reorder)
						It.Value() = Value;

						// mark that the key was found and the add has been processed
						bHandledWithKey = true;
						break;
					}
				}
			}
		}
	}

	return bHandledWithKey;
}

bool FConfigSection::GetString(const TCHAR* Key, FString& Value) const
{
	if (const FConfigValue* ConfigValue = Find(FName(Key)))
	{
		Value = ConfigValue->GetValue();
		return true;
	}

	return false;
}

bool FConfigSection::GetText(const TCHAR* Section, const TCHAR* Key, FText& Value) const
{
	if (const FConfigValue* ConfigValue = Find(FName(Key)))
	{
		return FTextStringHelper::ReadFromBuffer(*ConfigValue->GetValue(), Value, Section) != nullptr;
	}

	return false;
}

bool FConfigSection::GetInt(const TCHAR* Key, int32& Value) const
{
	FString Text;
	if (GetString(Key, Text))
	{
		Value = FCString::Atoi(*Text);
		return true;
	}
	return false;
}

bool FConfigSection::GetUInt(const TCHAR* Key, uint32& Value) const
{
	FString Text;
	if (GetString(Key, Text))
	{
		Value = static_cast<uint32>(FCString::Atoi(*Text));
		return true;
	}
	return false;
}

bool FConfigSection::GetFloat(const TCHAR* Key, float& Value) const
{
	FString Text;
	if (GetString(Key, Text))
	{
		Value = FCString::Atof(*Text);
		return true;
	}
	return false;
}

bool FConfigSection::GetDouble(const TCHAR* Key, double& Value) const
{
	FString Text;
	if (GetString(Key, Text))
	{
		Value = FCString::Atof(*Text);
		return true;
	}
	return false;
}

bool FConfigSection::GetInt64(const TCHAR* Key, int64& Value) const
{
	FString Text;
	if (GetString(Key, Text))
	{
		Value = FCString::Atoi64(*Text);
		return true;
	}
	return false;
}

bool FConfigSection::GetBool(const TCHAR* Key, bool& Value) const
{
	FString Text;
	if (GetString(Key, Text))
	{
		Value = FCString::ToBool(*Text);
		return true;
	}
	return false;
}

int32 FConfigSection::GetArray(const TCHAR* Key, TArray<FString>& Value) const
{
	const FName KeyName(Key);
	Value.Empty();

	MultiFind(KeyName, Value, true);

	// if we have values, or we were initialized to empty
	return Value.Num() || EmptyInitializedKeys.Contains(KeyName);
}

// Look through the file's per object config ArrayOfStruct keys and see if this section matches
template<typename SectionType>
void FixupArrayOfStructKeysForSection(SectionType* Section, const FString& SectionName, const TMap<FString, TMap<FName, FString> >& PerObjectConfigKeys)
{
	for (TMap<FString, TMap<FName, FString> >::TConstIterator It(PerObjectConfigKeys); It; ++It)
	{
		if (SectionName.EndsWith(It.Key()))
		{
			for (TMap<FName, FString>::TConstIterator It2(It.Value()); It2; ++It2)
			{
				Section->ArrayOfStructKeys.Add(It2.Key(), It2.Value());
			}
		}
	}
}


static FConfigCommandStream CalculateDiff(const FConfigFile& First, const FConfigFile& Second, const FString& SingleSection=FString(), const FString& SingleProperty=FString())
{
	FConfigCommandStream Diff;

	TArray<FString> SecondSectionKeys;
	Second.GetKeys(SecondSectionKeys);

	// loop over sections in the first file - since we are diffing to entries in a hierarchy, eveyrthing in first is in second (but not vice versa, as second can have new sections)
	for (const TPair<FString, FConfigSection>& FirstSectionIt : First)
	{
		// remove from SecondSectionKeys so that it only has what's only in Second
		SecondSectionKeys.Remove(FirstSectionIt.Key);

		// find the matching sections (expected they will both have)
		const FConfigSection* FirstSection = &FirstSectionIt.Value;
		const FConfigSection* SecondSection = Second.FindSection(FirstSectionIt.Key);
		FConfigCommandStreamSection* NewSection = nullptr;

		TSet<FName> FirstKeys;
		TSet<FName> SecondKeys;
		FirstSection->GetKeys(FirstKeys);
		if (SecondSection)
		{
			SecondSection->GetKeys(SecondKeys);
		}

		for (FName FirstKey : FirstKeys)
		{
			TArray<FConfigValue> FirstValues;
			TArray<FConfigValue> SecondValues;

			// remove the key from second, since we will already have processed it if it's in both
			SecondKeys.Remove(FirstKey);

			FirstSection->MultiFind(FirstKey, FirstValues, true);
			if (SecondSection != nullptr)
			{
				SecondSection->MultiFind(FirstKey, SecondValues, true);
			}
			if (SecondValues.Num() == 0)
			{
				if (NewSection == nullptr)
				{
					NewSection = Diff.FindOrAddSectionInternal(FirstSectionIt.Key);
				}

				// @todo: do we clear, or remove every value with -? this is hard to decide
				NewSection->Emplace(FirstKey, FConfigValue(TEXT("__ClearArray__"), FConfigValue::EValueType::Clear));
			}

			for (const FConfigValue& FirstValue : FirstValues)
			{
				FString FirstExpandedValue = FirstValue.GetSavedValueForWriting();

				bool bIsArray = FirstValue.ValueType == FConfigValue::EValueType::ArrayCombined || FirstValues.Num() > 1 || SecondValues.Num() > 1;

				bool bFound = false;
				for (auto It = SecondValues.CreateIterator(); It; ++It)
				{
					// if the second array doesn't have the value, then we need to remove it in the diff
					// if it found, remove it from the second array, so then the second is only what was added
					if (FirstExpandedValue == It->GetSavedValueForWriting())
					{
						It.RemoveCurrent();
						bFound = true;
						break;
					}
				}

				if (!bFound)
				{
					if (NewSection == nullptr)
					{
						NewSection = Diff.FindOrAddSectionInternal(FirstSectionIt.Key);
					}

					if (bIsArray)
					{
						// add this remove value to the diff
						NewSection->Emplace(FirstKey, FConfigValue(FirstValue.GetSavedValueForWriting(), FConfigValue::EValueType::Remove));
					}
					else
					{
						// if the second one set the value, and it wasn't found above, that means it's different, so use ::Set
						if (SecondValues.Num() > 0)
						{
							NewSection->Emplace(FirstKey, FConfigValue(SecondValues[0].GetSavedValueForWriting(), FConfigValue::EValueType::Set));
							SecondValues.Empty();
						}
						// if the second didn't set it, then we want to remove the key, so we go back to defaults
						else
						{
							NewSection->Emplace(FirstKey, FConfigValue(FirstExpandedValue, FConfigValue::EValueType::Clear));
						}
					}
				}
			}

			// the values that are left all need to be added to the diff
			for (const FConfigValue& SecondValue : SecondValues)
			{
				if (NewSection == nullptr)
				{
					NewSection = Diff.FindOrAddSectionInternal(FirstSectionIt.Key);
				}

				// add this value to the diff as a set (if one value) or arrayadd if there are multiple
				FConfigValue::EValueType Type = (FirstValues.Num() == 0 && SecondValues.Num() == 1 && SecondValues[0].ValueType != FConfigValue::EValueType::ArrayCombined) ?
					FConfigValue::EValueType::Set : FConfigValue::EValueType::ArrayAddUnique;
				NewSection->Emplace(FirstKey, FConfigValue(SecondValue.GetSavedValueForWriting(), Type));
			}
		}

		// now go over SecondKeys which will only have keys not in first section
		if (SecondSection != nullptr)
		{
			for (FName SecondKey : SecondKeys)
			{
				if (NewSection == nullptr)
				{
					NewSection = Diff.FindOrAddSectionInternal(FirstSectionIt.Key);
				}

				TArray<FConfigValue> SecondValues;
				SecondSection->MultiFind(SecondKey, SecondValues, true);

				FConfigValue::EValueType Type = (SecondValues.Num() == 1) ? FConfigValue::EValueType::Set : FConfigValue::EValueType::ArrayAddUnique;
				for (const FConfigValue& SecondValue : SecondValues)
				{
					NewSection->Emplace(SecondKey, FConfigValue(SecondValue.GetSavedValueForWriting(), Type));
				}
			}
		}
	}
	
	// finally sections that are only in second need to be copied added in to the diff
	for (const FString& SecondSectionKey : SecondSectionKeys)
	{
		const FConfigSection& SecondSection = *Second.FindSection(SecondSectionKey);
		FConfigCommandStreamSection* NewSection = Diff.FindOrAddSectionInternal(SecondSectionKey);
		
		TSet<FName> SecondKeys;
		SecondSection.GetKeys(SecondKeys);

		for (FName SecondKey : SecondKeys)
		{
			TArray<FConfigValue> SecondValues;
			SecondSection.MultiFind(SecondKey, SecondValues, true);

			FConfigValue::EValueType Type = (SecondValues.Num() == 1) ? FConfigValue::EValueType::Set : FConfigValue::EValueType::ArrayAddUnique;
			for (const FConfigValue& SecondValue : SecondValues)
			{
				NewSection->Emplace(SecondKey, FConfigValue(SecondValue.GetSavedValueForWriting(), Type));

			}
		}
	}
	
	return Diff;
}

template<typename FileType>
static bool BuildOutputString(FString& String, const FileType& FileToWrite)
{
	for (auto& Section : FileToWrite)
	{
		String.Append(TEXT("["));
		String.Append(Section.Key);
		String.Append(TEXT("]" LINE_TERMINATOR_ANSI));
		
		for (const TPair<FName, FConfigValue>& Value : Section.Value)
		{
#if CONFIG_CAN_SAVE_COMMENTS
			if (Value.Value.Comment.Len() > 0)
			{
				String.Append(Value.Value.Comment);
				String.Append(LINE_TERMINATOR);
			}
#endif
			if (Value.Value.ValueType != FConfigValue::EValueType::Set)
			{
				const TCHAR* Cmd = CommandLookup.FindKey(Value.Value.ValueType);
				String.AppendChar(*Cmd);
			}
			FConfigFile::AppendExportedPropertyLine(String, Value.Key.ToString(), Value.Value.GetSavedValueForWriting());
		}
		String.Append(LINE_TERMINATOR);
	}
	
	return true;
}

static bool BuildDiffOutputString(FString& String, const FConfigFile& FileToWrite, const FConfigFile& FileToDiffAgainst)
{
	FConfigCommandStream Diff = CalculateDiff(FileToDiffAgainst, FileToWrite);
	return BuildOutputString(String, Diff);
}

static bool AreWritesAllowedGlobally()
{
	bool bNoWrite = FParse::Param(FCommandLine::Get(), TEXT("nowrite")) ||
		// It can be useful to save configs with multiprocess if they are given INI overrides
		(FParse::Param(FCommandLine::Get(), TEXT("Multiprocess")) && !FParse::Param(FCommandLine::Get(), TEXT("MultiprocessSaveConfig")));
		 
	return !bNoWrite;
}

static bool SaveBranch(FConfigBranch& Branch)
{
	if (!Branch.InMemoryFile.Dirty || Branch.InMemoryFile.NoSave || !AreWritesAllowedGlobally())
	{
		return true;
	}

	FString Output;
	bool bBuiltString;
	if (GUseNewSaveTracking == 1)
	{
		bBuiltString = BuildOutputString(Output, Branch.SavedLayer);
	}
	else if (GUseNewSaveTracking == 2)
	{
		bBuiltString = BuildDiffOutputString(Output, Branch.InMemoryFile, Branch.FinalCombinedLayers);
	}
	else
	{
		Branch.InMemoryFile.WriteToString(Output, Branch.IniPath);
		bBuiltString = true;
	}
		
	if (bBuiltString && Output.Len() > 0)
	{
		Output = FString(TEXT(";METADATA=(Diff=true, UseCommands=true)" LINE_TERMINATOR_ANSI)) + Output;
		return SaveConfigFileWrapper(*Branch.IniPath, Output);
	}

	// delete any old crusty saved ini files from before we disabled most sections' from writing
	IFileManager::Get().Delete(*Branch.IniPath);

	// return true that we saved, even if we didn't need to write out anything
	return true;
}


/*-----------------------------------------------------------------------------
	FConfigFile
-----------------------------------------------------------------------------*/
FConfigFile::FConfigFile()
    : Dirty( false )
    , NoSave( false )
    , bHasPlatformName( false )
    , bPythonConfigParserMode( false )
    , bCanSaveAllSections( true )
    , Name( NAME_None )
{
	FCoreDelegates::TSOnFConfigCreated().Broadcast(this);
}

FConfigFile::~FConfigFile()
{
	// this destructor can run at file scope, static shutdown

	if ( !GExitPurge )
	{
		FCoreDelegates::TSOnFConfigDeleted().Broadcast(this);
	}

#if UE_WITH_CONFIG_TRACKING 
	if (FileAccess)
	{
		FileAccess->ConfigFile = nullptr;
	}
#endif

	Cleanup();
}

FConfigFile::FConfigFile(const FConfigFile& Other)
{
	*this = Other;
}

FConfigFile::FConfigFile(FConfigFile&& Other)
{
	*this = MoveTemp(Other);
}

FConfigFile& FConfigFile::operator=(const FConfigFile& Other)
{
	UE::TWriteScopeLock ScopeLock(ConfigFileMapLock);
	this->FConfigFileMap::operator=(Other);
	Dirty = Other.Dirty;
	NoSave = Other.NoSave;
	bHasPlatformName = Other.bHasPlatformName;
	bPythonConfigParserMode = Other.bPythonConfigParserMode;
	bCanSaveAllSections = Other.bCanSaveAllSections;

	// LoadType is not copied; each FConfigFile has to set it itself

	Name = Other.Name;
	PlatformName = Other.PlatformName;
	Tag = Other.Tag;
	Branch = Other.Branch;

	// @todo branch - remove this right?
#if ALLOW_INI_OVERRIDE_FROM_COMMANDLINE
	CommandlineOptions = Other.CommandlineOptions;
#endif // ALLOW_INI_OVERRIDE_FROM_COMMANDLINE

	PerObjectConfigArrayOfStructKeys = Other.PerObjectConfigArrayOfStructKeys;

	// FileAccess is not copied; each FConfigFile has to set it itself

	// Update the FileAccess pointers on all Sections and Values that were assigned
#if UE_WITH_CONFIG_TRACKING
	UE::ConfigAccessTracking::FFile* LocalFileAccess = GetFileAccess();
	for (TMap<FString, FConfigSection>::TIterator SectionIterator(*this); SectionIterator; ++SectionIterator)
	{
		UE::ConfigAccessTracking::FSection* SectionAccess = nullptr;
		if (LocalFileAccess)
		{
			SectionAccess = new UE::ConfigAccessTracking::FSection(*LocalFileAccess, FStringView(SectionIterator->Key));
		}
		SectionIterator->Value.SectionAccess = SectionAccess;
		for (TPair<FName, FConfigValue>& ValuePair : SectionIterator->Value)
		{
			ValuePair.Value.SetSectionAccess(SectionAccess);
		}
	}
#endif

	return *this;
}

FConfigFile& FConfigFile::operator=(FConfigFile&& Other)
{
	UE::TWriteScopeLock ScopeLock(ConfigFileMapLock);
	this->FConfigFileMap::operator=(MoveTemp(Other));
	Dirty = Other.Dirty;
	NoSave = Other.NoSave;
	bHasPlatformName = Other.bHasPlatformName;
	bCanSaveAllSections = Other.bCanSaveAllSections;

	// LoadType is not copied; each FConfigFile has to set it itself

	Name = MoveTemp(Other.Name);
	PlatformName = MoveTemp(Other.PlatformName);
	Tag = MoveTemp(Other.Tag);
	Branch = MoveTemp(Other.Branch);

	// @todo branch - remove this right?
#if ALLOW_INI_OVERRIDE_FROM_COMMANDLINE
	CommandlineOptions = MoveTemp(Other.CommandlineOptions);
#endif // ALLOW_INI_OVERRIDE_FROM_COMMANDLINE

	PerObjectConfigArrayOfStructKeys = MoveTemp(Other.PerObjectConfigArrayOfStructKeys);

	// FileAccess is not copied; each FConfigFile has to set it itself

	// Update the FileAccess pointers on all Sections and Values that were assigned
#if UE_WITH_CONFIG_TRACKING
	UE::ConfigAccessTracking::FFile* LocalFileAccess = GetFileAccess();
	for (TMap<FString, FConfigSection>::TIterator SectionIterator(*this); SectionIterator; ++SectionIterator)
	{
		UE::ConfigAccessTracking::FSection* SectionAccess = nullptr;
		if (LocalFileAccess)
		{
			SectionAccess = new UE::ConfigAccessTracking::FSection(*LocalFileAccess, FStringView(SectionIterator->Key));
		}
		SectionIterator->Value.SectionAccess = SectionAccess;
		for (TPair<FName, FConfigValue>& ValuePair : SectionIterator->Value)
		{
			ValuePair.Value.SectionAccess = SectionAccess;
		}
	}
#endif

	return *this;
}

UE_AUTORTFM_ALWAYS_OPEN void FConfigFile::Cleanup()
{
	Empty();
}

#if UE_WITH_CONFIG_TRACKING 
void FConfigFile::SuppressReporting()
{
	LoadType = UE::ConfigAccessTracking::ELoadType::SuppressReporting;
	if (FileAccess)
	{
		FileAccess->ConfigFile = nullptr;
		FileAccess.SafeRelease();
	}
}

UE::ConfigAccessTracking::FFile* FConfigFile::GetFileAccess() const
{
	if (!FileAccess)
	{
		if (LoadType == UE::ConfigAccessTracking::ELoadType::SuppressReporting)
		{
			return nullptr;
		}
		FileAccess = new UE::ConfigAccessTracking::FFile(this);
	}
	return FileAccess.GetReference();
}
#endif

bool FConfigFile::operator==( const FConfigFile& Other ) const
{
	UE::TReadScopeLock ScopeLock(ConfigFileMapLock);

	if ( Pairs.Num() != Other.Pairs.Num() )
		return 0;

	for ( TMap<FString,FConfigSection>::TConstIterator It(*this), OtherIt(Other); It && OtherIt; ++It, ++OtherIt)
	{
		if ( It.Key() != OtherIt.Key() )
			return 0;

		if ( It.Value() != OtherIt.Value() )
			return 0;
	}

	return 1;
}

bool FConfigFile::operator!=( const FConfigFile& Other ) const
{
	return ! (FConfigFile::operator==(Other));
}

FConfigSection* FConfigFile::FindOrAddSectionInternal(const FString& SectionName)
{
	FConfigSection* Section = FindInternal(SectionName);
	if (Section == nullptr)
	{
		UE::ConfigAccessTracking::FSection* SectionAccess = nullptr;
#if UE_WITH_CONFIG_TRACKING
		UE::ConfigAccessTracking::FFile* LocalFileAccess = GetFileAccess();
		if (LocalFileAccess)
		{
			SectionAccess = new UE::ConfigAccessTracking::FSection(*LocalFileAccess, FStringView(SectionName));
		}
#endif
		Section = &Add(SectionName, FConfigSection(SectionAccess));
	}
	return Section;
}

const FConfigSection* FConfigFile::FindOrAddConfigSection(const FString& SectionName)
{
	return FindOrAddSectionInternal(SectionName);
}

bool FConfigFile::Combine(const FString& Filename, bool bHandleSymbolCommands)
{
	return FillFileFromDisk(Filename, bHandleSymbolCommands);
}

void FConfigFile::Shrink()
{
#if !UE_BUILD_SHIPPING
	extern double GConfigShrinkTime;
	if (IsInGameThread()) GConfigShrinkTime -= FPlatformTime::Seconds();
#endif

	UE::TWriteScopeLock ScopeLock(ConfigFileMapLock);
	FConfigFileMap::Shrink();

	for (FConfigFileMap::TIterator It(*this); It; ++It)
	{
		It.Value().Shrink();
	}

	PerObjectConfigArrayOfStructKeys.Shrink();
	for (auto& Pair : PerObjectConfigArrayOfStructKeys)
	{
		Pair.Value.Shrink();
	}

#if !UE_BUILD_SHIPPING
	if (IsInGameThread()) GConfigShrinkTime += FPlatformTime::Seconds();
#endif
}

// Assumes GetTypeHash(AltKeyType) matches GetTypeHash(KeyType)
template<class KeyType, class ValueType, class AltKeyType>
ValueType& FindOrAddHeterogeneous(TMap<KeyType, ValueType>& Map, const AltKeyType& Key)
{
	checkSlow(GetTypeHash(KeyType(Key)) == GetTypeHash(Key));
	ValueType* Existing = Map.FindByHash(GetTypeHash(Key), Key);
	return Existing ? *Existing : Map.Emplace(KeyType(Key));
}

namespace
{

// don't allow warning until all redirects are read in
bool GAllowConfigRemapWarning = false;

// either show an editor warning, or just write to log for non-editor
void LogOrEditorWarning(const FText& Msg, const FString& PartialKey, const FString& File)
{
	if (!GAllowConfigRemapWarning)
	{
		return;
	}
	
	if (GIsEditor)
	{
		static TSet<FString> AlreadyWarnedKeys;
		
		FString AbsPath = IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*File);
		
		// make sure we haven't warned about this yet
		FString Key = PartialKey + AbsPath;
		if (AlreadyWarnedKeys.Contains(Key))
		{
			return;
		}
		AlreadyWarnedKeys.Add(Key);
		
		FMessageLog EditorErrors("EditorErrors");
		TSharedRef<FTokenizedMessage> Message = EditorErrors.Message(EMessageSeverity::Warning);
		if (File.EndsWith(TEXT(".ini")))
		{
			Message->AddToken(FURLToken::Create(FString::Printf(TEXT("file://%s"), *AbsPath), LOCTEXT("DeprecatedConfig_URLCLick", "Click to open file")));
		}
		Message->AddToken(FTextToken::Create(Msg));
		EditorErrors.Notify();
	}

	// always spit to log
	UE_LOGF(LogConfig, Warning, "%ls", *Msg.ToString());
}

// warn about a section name that's deprecated
void WarnAboutSectionRemap(const FString& OldValue, const FString& NewValue, const FString& File)
{
	if (!GAllowConfigRemapWarning)
	{
		return;
	}

	FFormatNamedArguments Arguments;
	Arguments.Add(TEXT("OldValue"), FText::FromString(OldValue));
	Arguments.Add(TEXT("NewValue"), FText::FromString(NewValue));
	Arguments.Add(TEXT("File"), FText::FromString(File));
	FText Msg = FText::Format(LOCTEXT("DeprecatedConfig", "Found a deprecated ini section name in {File}. Search for [{OldValue}] and replace with [{NewValue}]"), Arguments);
	
	FString Key = OldValue;
	if (!IsInGameThread())
	{
		AsyncTask( ENamedThreads::GameThread, [Msg, Key, File]() { LogOrEditorWarning(Msg, Key, File); });
	}
	else
	{
		LogOrEditorWarning(Msg, Key, File);
	}
}

// warn about a key that's deprecated
static void WarnAboutKeyRemap(const FString& OldValue, const FString& NewValue, const FString& Section, const FString& File)
{
	FFormatNamedArguments Arguments;
	Arguments.Add(TEXT("OldValue"), FText::FromString(OldValue));
	Arguments.Add(TEXT("NewValue"), FText::FromString(NewValue));
	Arguments.Add(TEXT("Section"), FText::FromString(Section));
	Arguments.Add(TEXT("File"), FText::FromString(File));
	FText Msg = FText::Format(LOCTEXT("DeprecatedConfigKey", "Found a deprecated ini key name in {File}. Search for [{OldValue}] and replace with [{NewValue}]"), Arguments);
	
	FString Key = OldValue+Section;
	if (!IsInGameThread())
	{
		AsyncTask( ENamedThreads::GameThread, [Msg, Key, File]() { LogOrEditorWarning(Msg, Key, File); });
	}
	else
	{
		LogOrEditorWarning(Msg, Key, File);
	}
}

}


#if ALLOW_INI_OVERRIDE_FROM_COMMANDLINE

/** A collection of identifiers which will help us parse the commandline opions. */
namespace CommandlineOverrideSpecifiers
{
	// -ini:IniName:[Section1]:Key1=Value1,[Section2]:Key2=Value2
	const auto& IniFileOverrideIdentifier = TEXT("-iniFile=");
	const auto& IniSwitchIdentifier       = TEXT("-ini:");
	const auto& IniNameEndIdentifier      = TEXT(":[");
	const auto& SectionStartIdentifier    = TEXT("[");
	const auto& PropertyStartIdentifier   = TEXT("]:");
	const TCHAR PropertySeperator         = TEXT(',');
	const auto& CustomConfigIdentifier    = TEXT("-CustomConfig=");
}

#endif

static bool OverrideFileFromCommandline(FString& InOutFilename)
{
#if ALLOW_INI_OVERRIDE_FROM_COMMANDLINE
	// look for this filename on the commandline in the format:
	//		-iniFile=<PatFile1>,<PatFile2>,<PatFile3>
	// for example:
	//		-iniFile=D:\UE\QAGame\Config\Windows\WindowsDeviceProfiles.ini
	//
	//		Description:
	//          The QAGame\Config\Windows\WindowsDeviceProfiles.ini contained in the pak file will
	//          be replace with D:\UE\QAGame\Config\Windows\WindowsDeviceProfiles.ini.

	//			Note: You will need the same base file path for this to work. If you
	//                want to override Engine/Config/BaseEngine.ini, you will need to place the override file
	//                under the same folder structure.
	//          Ex1: D:\<some_folder>\Engine\Config\BaseEngine.ini
	//			Ex2: D:\<some_folder>\QAGame\Config\Windows\WindowsEngine.ini
	static bool bHasCachedData = false;

	static TArray<FString> Files;
	if (UNLIKELY(!bHasCachedData))
	{
		FString StagedFilePaths;
		if (FParse::Value(FCommandLine::Get(), CommandlineOverrideSpecifiers::IniFileOverrideIdentifier, StagedFilePaths, false))
		{
			StagedFilePaths.ParseIntoArray(Files, TEXT(","), true);
		}

		bHasCachedData = true;
	}

	if (Files.Num() > 0)
	{
		FString RelativePath = InOutFilename;
		if (FPaths::IsUnderDirectory(RelativePath, FPaths::RootDir()))
		{
			FPaths::MakePathRelativeTo(RelativePath, *FPaths::RootDir());

			for (int32 Index = 0; Index < Files.Num(); Index++)
			{
				FString NormalizedOverride = Files[Index];
				FPaths::NormalizeFilename(NormalizedOverride);
				if (NormalizedOverride.EndsWith(RelativePath))
				{
					InOutFilename = Files[Index];
					UE_LOGF(LogConfig, Warning, "Loading override ini file: %ls ", *Files[Index]);
					return true;
				}
			}
		}
	}
#endif

	return false;
}

bool FConfigFile::ApplyFile(const FConfigCommandStream* File)
{
	// walk over the section in the file to apply
	for (const TPair<FString, FConfigCommandStreamSection>& SourceSectionIt : *File)
	{
		TSet<FName> RemovedKeys;

		const FConfigCommandStreamSection* SourceSection = &SourceSectionIt.Value;
		FConfigSection* TargetSection = FindOrAddSectionInternal(SourceSectionIt.Key);

		// make sure the CurrentSection has any of the special ArrayOfStructKeys added
		FixupArrayOfStructKeysForSection(TargetSection, SourceSectionIt.Key, PerObjectConfigArrayOfStructKeys);

		for (const TPair<FName, FConfigValue>& SourceValue : *SourceSection)
		{
			FName Key = SourceValue.Key;
			FString Value = SourceValue.Value.GetSavedValue();
			FConfigValue::EValueType ValueType = SourceValue.Value.ValueType;

			// Saved config files would be read in, then entries not in the Saved file that were in the Static layers
			// would be merged into the final COnfigFile - this emulates that by removing the entries we are replacing
			// before reading any in (we can't instantly tell if there will be 1 or N instances of the key)
			if (File->bIsSavedConfigFile && !RemovedKeys.Contains(Key))
			{
				TargetSection->Remove(Key);
				RemovedKeys.Add(Key);
			}
			
			//// the value will be Combined once applied to the file
			//Value.ValueType = FConfigValue::EValueType::Combined;

			ProcessCommand(TargetSection, SourceSectionIt.Key, ValueType, Key, MoveTemp(Value));
		}
	}

	return true;
}

void FConfigFile::ProcessCommand(FConfigSection* Section, FStringView SectionName, FConfigValue::EValueType Command, FName Key, FString&& Value)
{
	switch (Command)
	{
		case FConfigValue::EValueType::Set:
			// First see if this can be processed as an array of keyed structs command
			if (!Section->HandleArrayOfKeyedStructsCommand(Key, MoveTemp(Value)))
			{
				// Add if not present and replace if present.
				FConfigValue* ConfigValue = Section->Find(Key);
				if (!ConfigValue)
				{
					Section->Add(Key, FConfigValue(Section, Key, MoveTemp(Value)));
				}
				else
				{
					*ConfigValue = MoveTemp(Value);
				}
			}
			break;
		case FConfigValue::EValueType::ArrayAddUnique:
			// Add if not already present.
			Section->HandleAddCommand(Key, MoveTemp(Value), false);
			break;
		case FConfigValue::EValueType::ArrayAdd:
			// Add even if already present.
			Section->HandleAddCommand(Key, MoveTemp(Value), true);
			break;
		case FConfigValue::EValueType::Remove:
			// Remove if present.
			Section->RemoveSingleStable(Key, Value);
			break;
		case FConfigValue::EValueType::Clear:
			// Remove if present.
			Section->RemoveStable(Key);
			
			// clear any empty initialization, which will reset it to using code defaults if nothing else adds anything else
			Section->EmptyInitializedKeys.Remove(Key);
			break;
		case FConfigValue::EValueType::InitializeToEmpty:
			// track a key to show uniqueness for arrays of structs
			Section->EmptyInitializedKeys.Add(Key);
			
			// also clear any existing entries
			Section->RemoveStable(Key);
			break;
		case FConfigValue::EValueType::ArrayOfStructKey:
			// track a key to show uniqueness for arrays of structs
			Section->ArrayOfStructKeys.Add(Key, MoveTemp(Value));
			break;
		case FConfigValue::EValueType::POCArrayOfStructKey:
		{
			// track a key to show uniqueness for arrays of structs
			TMap<FName, FString>& POCKeys = FindOrAddHeterogeneous(PerObjectConfigArrayOfStructKeys, SectionName);
			POCKeys.Add(Key, MoveTemp(Value));
		}
		break;
		default: unimplemented();
	}
}

#if UE_WITH_CONFIG_TRACKING
static void ConditionalInitializeLoadType(FConfigFile* File, UE::ConfigAccessTracking::ELoadType LoadType,
	FName FileName)
{
	if (File->LoadType == UE::ConfigAccessTracking::ELoadType::Uninitialized)
	{
		File->LoadType = LoadType;
	}
	if (File->Name.IsNone())
	{
		File->Name = FileName;
	}
}
static void ConditionalInitializeLoadType(FConfigCommandStream* File, UE::ConfigAccessTracking::ELoadType LoadType,
	FName FileName)
{
}
#endif

template<typename FileType>
void FillFileFromBuffer(FileType* File, FStringView Buffer, bool bHandleSymbolCommands, const FString& FileHint)
{
	LLM_SCOPE(ELLMTag::ConfigSystem);

	static const FName ConfigFileClassName = "ConfigFile";
	const FName FileName = FName(*FileHint);
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH_FNAME(FileName, ELLMTagSet::Assets);
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH_FNAME(ConfigFileClassName, ELLMTagSet::AssetClasses);
	UE_TRACE_METADATA_SCOPE_ASSET_FNAME(FileName, ConfigFileClassName, FileName);

#if UE_WITH_CONFIG_TRACKING
	ConditionalInitializeLoadType(File, UE::ConfigAccessTracking::ELoadType::LocalSingleIniFile, FileName);
#endif

	const TCHAR* Ptr = Buffer.GetData();
	
	using SectionType = typename FileType::SectionType;
	SectionType* CurrentSection = nullptr;
	
	FString CurrentSectionName;
	FName CurrentKeyName;
	TMap<FString, FString>* CurrentKeyRemap = nullptr;
	TStringBuilder<128> TheLine;
	FString ProcessedValue;
	bool Done = false;
	bool bHasHandledMetadata = false;

	FParse::ELineExtendedFlags Flags = 
		FParse::ELineExtendedFlags::SwallowDoubleSlashComments |
		FParse::ELineExtendedFlags::AllowBracketedMultiline |
		FParse::ELineExtendedFlags::AllowEscapedEOLMultiline |
		FParse::ELineExtendedFlags::SwallowExtraEOLs;
	if (File->bPythonConfigParserMode)
	{
		Flags = FParse::ELineExtendedFlags::OldExactMode;
	}
	
	while (!Done && Ptr - Buffer.GetData() < Buffer.Len())
	{
		// Advance past new line characters
		while (*Ptr == '\r' || *Ptr == '\n')
		{
			Ptr++;
		}

		// read the next line
		int32 LinesConsumed = 0;
		FParse::LineExtended(&Ptr, /* reset */ TheLine, LinesConsumed, Flags);
		if (Ptr == nullptr || *Ptr == 0)
		{
			Done = true;
		}
		int32 LineLen = TheLine.Len();
		TCHAR* Start = const_cast<TCHAR*>(*TheLine);

		// Strip trailing spaces from the current line
		while( *Start && FChar::IsWhitespace(Start[LineLen-1]) )
		{
			Start[LineLen-1] = TEXT('\0');
			LineLen--;
		}

		//@todo (UE-214768) Comment this back in
		// if (!bHasHandledMetadata)
		// {
		// 	bHasHandledMetadata = true;
			
		// 	if (FCString::Strnicmp(Start, TEXT(";METADATA="), 10) == 0)
		// 	{
		// 		FString MetadataStruct(Start + 10);
		// 		FString MetadataValue;
		// 		ExtractPropertyValue(MetadataStruct, TEXT("UseCommands="), MetadataValue);
		// 		if (MetadataValue.Len() > 0)
		// 		{
		// 			bHandleSymbolCommands = FCString::ToBool(*MetadataValue);
		// 		}
				
		// 		// move on to the next line
		// 		continue;
		// 	}
			
		// }
		// If the first character in the line is [ and last char is ], this line indicates a section name
		if( *Start=='[' && Start[LineLen-1]==']' )
		{
			// Remove the brackets
			Start++;
			LineLen--;
			Start[LineLen-1] = TEXT('\0');
			LineLen--;

			// If we don't have an existing section by this name, add one
			CurrentSectionName = Start;
			CurrentKeyName = NAME_None;
			
			// lookup to see if there is an entry in the SectionName remap
			const FString* FoundRemap;
			if ((FoundRemap = SectionRemap.Find(CurrentSectionName)) != nullptr)
			{
				// show warning in editor
				WarnAboutSectionRemap(CurrentSectionName, *FoundRemap, FileHint);
				
				CurrentSectionName = *FoundRemap;
			}
			if (CurrentSection)
			{
				CurrentSection->Shrink();
			}
			CurrentSection = File->FindOrAddSectionInternal(CurrentSectionName);

			// look to see if there is a set of key remaps for this section
			CurrentKeyRemap = KeyRemap.Find(CurrentSectionName);

			// make sure the CurrentSection has any of the special ArrayOfStructKeys added
			if (File->PerObjectConfigArrayOfStructKeys.Num() > 0)
			{
				FixupArrayOfStructKeysForSection(CurrentSection, CurrentSectionName, File->PerObjectConfigArrayOfStructKeys);
			}
		}

		// Otherwise, if we're currently inside a section, and we haven't reached the end of the stream
		else if( CurrentSection && *Start )
		{
			TCHAR* Value = 0;
			
			// ignore [comment] lines that start with ;
			if(*Start != (TCHAR)';')
			{
				// If we're in python mode and the line starts with whitespace
				// then we should consider it a part of the prior key
				if (File->bPythonConfigParserMode && !CurrentKeyName.IsNone() && FChar::IsWhitespace(*Start))
				{
					Value = Start;
				}
				else
				{
					Value = FCString::Strstr(Start,TEXT("="));
				}
			}

			// Ignore any lines that don't contain a key-value pair
			if( Value )
			{
				SectionType* OriginalCurrentSection = CurrentSection;

				// determine how this line will be merged
				// when we don't want commands, the default action is to add new entries (this is for standalone ini files that have arrays,
				// without any + cmds) - there's no difference between a single value and an array of 1 (in terms of the Config system)
				FConfigValue::EValueType Command = FConfigValue::EValueType::ArrayAdd;

				// Value will be Start in the python configparser extending case in which case
				// we want to continue using the CurrentKeyName
				if (Value != Start)
				{
					// Terminate the property name, advancing past the =
					*Value++ = TEXT('\0');
					// update LineLen since we just chopped it
					LineLen = (int32)(Value - Start - 1);

					// strip leading whitespace from the property name
					while (*Start && FChar::IsWhitespace(*Start))
					{
						Start++;
						LineLen--;
					}

					// ~ is a packaging and should be skipped at runtime
					if (Start[0] == '~')
					{
						Start++;
						LineLen--;
					}

					if (bHandleSymbolCommands)
					{
						TCHAR Cmd = Start[0];
						if (Cmd == '+' || Cmd == '-' || Cmd == '.' || Cmd == '!' || Cmd == '@' || Cmd == '*' || Cmd == '^')
						{
							Start++;
							LineLen--;
						}
						else
						{
							Cmd = TEXT('\0');
						}
					
						// turn into a command
						FConfigValue::EValueType* Lookup = CommandLookup.Find(Cmd);
						if (Lookup == nullptr)
						{
							UE_LOGF(LogConfig, Log, "Found unknown ini command %c in an ini", Cmd);
							continue;
						}
						Command = *Lookup;
					}

					// Strip trailing spaces from the property name.
					while (*Start && FChar::IsWhitespace(Start[LineLen-1]))
					{
						Start[LineLen-1] = TEXT('\0');
						LineLen--;
					}

					const TCHAR* KeyName = Start;
					// look up for key remap
					if (CurrentKeyRemap != nullptr)
					{
						const FString* FoundRemap;
						if ((FoundRemap = CurrentKeyRemap->Find(KeyName)) != nullptr)
						{
							WarnAboutKeyRemap(KeyName, *FoundRemap, CurrentSectionName, FileHint);

							// the Remap will not ever reallocate, so we can just point right into the FString
							KeyName = **FoundRemap;

							// look for a section:name remap
							int32 ColonLoc;
							if (FoundRemap->FindChar(':', ColonLoc))
							{
								// find or create a section for name before the :
								CurrentSection = File->FindOrAddSectionInternal(*FoundRemap->Mid(0, ColonLoc));
								// the name can still point right into the FString, but right after the :
								KeyName = **FoundRemap + ColonLoc + 1;
							}
						}
					}

					CurrentKeyName = FName(KeyName);
				}

				// Strip leading whitespace from the property value
				while ( *Value && FChar::IsWhitespace(*Value) )
				{
					Value++;
				}

				// strip trailing whitespace from the property value
				while( *Value && FChar::IsWhitespace(Value[FCString::Strlen(Value)-1]) )
				{
					Value[FCString::Strlen(Value)-1] = TEXT('\0');
				}

				ProcessedValue.Reset();

				// If this line is delimited by quotes
				if( *Value=='\"' )
				{
					FParse::QuotedString(Value, ProcessedValue);
				}
				else
				{
					ProcessedValue = Value;
				}

				File->ProcessCommand(CurrentSection, FStringView(CurrentSectionName), Command, CurrentKeyName, MoveTemp(ProcessedValue));
				
				// restore the current section, in case it was overridden
				CurrentSection = OriginalCurrentSection;

				// Mark as dirty so "Write" will actually save the changes.
				File->Dirty = true;
			}
		}
	}

	// Avoid memory wasted in array slack.
	File->Shrink();
}

template<typename FileType>
bool FillFileFromDisk(FileType* File, const FString& Filename, bool bHandleSymbolCommands)
{
	FString Text;

	FString FinalFileName = Filename;
	bool bFoundOverride = OverrideFileFromCommandline(FinalFileName);

	if (LoadConfigFileWrapper(*FinalFileName, Text, bFoundOverride))
	{
		FillFileFromBuffer(File, Text, bHandleSymbolCommands, Filename);
		return true;
	}

	checkf(!bFoundOverride, TEXT("Failed to Load config override %s"), *FinalFileName);
	return false;
}


void FConfigFile::FillFileFromBuffer(FStringView Buffer, bool bHandleSymbolCommands, const FString& FileHint)
{
	::FillFileFromBuffer(this, Buffer, bHandleSymbolCommands, FileHint);
}

bool FConfigFile::FillFileFromDisk(const FString& Filename, bool bHandleSymbolCommands)
{
	return ::FillFileFromDisk(this, Filename, bHandleSymbolCommands);
}

void FConfigFile::CombineFromBuffer(const FString& Buffer, const FString& FileHint, bool bHandleSymbolCommands)
{
	::FillFileFromBuffer(this, Buffer, bHandleSymbolCommands, FileHint);
}

/**
 * Process the contents of an .ini file that has been read into an FString
 *
 * @param Contents Contents of the .ini file
 */
void FConfigFile::ProcessInputFileContents(FStringView Contents, const FString& FileHint)
{
	::FillFileFromBuffer(this, Contents, false, FileHint);
}

void FConfigFile::Read( const FString& Filename )
{
	::FillFileFromDisk(this, Filename, false);
}

bool FConfigFile::ShouldExportQuotedString(const FString& PropertyValue)
{
	bool bEscapeNextChar = false;
	bool bIsWithinQuotes = false;

	// The value should be exported as quoted string if...
	const TCHAR* const DataPtr = *PropertyValue;
	for (const TCHAR* CharPtr = DataPtr; *CharPtr; ++CharPtr)
	{
		const TCHAR ThisChar = *CharPtr;
		const TCHAR NextChar = *(CharPtr + 1);

		const bool bIsFirstChar = CharPtr == DataPtr;
		const bool bIsLastChar = NextChar == 0;

		if (ThisChar == TEXT('"') && !bEscapeNextChar)
		{
			bIsWithinQuotes = !bIsWithinQuotes;
		}
		bEscapeNextChar = ThisChar == TEXT('\\') && bIsWithinQuotes && !bEscapeNextChar;

		// ... it begins or ends with a space (which is stripped on import)
		if (ThisChar == TEXT(' ') && (bIsFirstChar || bIsLastChar))
		{
			return true;
		}

		// ... it begins with a '"' (which would be treated as a quoted string)
		if (ThisChar == TEXT('"') && bIsFirstChar)
		{
			return true;
		}

		// ... it ends with a '\' (which would be treated as a line extension)
		if (ThisChar == TEXT('\\') && bIsLastChar)
		{
			return true;
		}

		// ... it contains unquoted '{' or '}' (which are stripped on import)
		if ((ThisChar == TEXT('{') || ThisChar == TEXT('}')) && !bIsWithinQuotes)
		{
			return true;
		}
		
		// ... it contains unquoted '//' (interpreted as a comment when importing)
		if ((ThisChar == TEXT('/') && NextChar == TEXT('/')) && !bIsWithinQuotes)
		{
			return true;
		}

		// ... it contains an unescaped new-line
		if (!bEscapeNextChar && (NextChar == TEXT('\r') || NextChar == TEXT('\n')))
		{
			return true;
		}
	}

	return false;
}

FString FConfigFile::GenerateExportedPropertyLine(const FString& PropertyName, const FString& PropertyValue)
{
	FString Out;
	AppendExportedPropertyLine(Out, PropertyName, PropertyValue);
	return Out;
}

void FConfigFile::AppendExportedPropertyLine(FString& Out, const FString& PropertyName, const FString& PropertyValue)
{
	// Append has been measured to be twice as fast as Appendf here
	Out.Append(PropertyName);

	Out.AppendChar(TEXT('='));

	if (FConfigFile::ShouldExportQuotedString(PropertyValue))
	{
		Out.AppendChar(TEXT('"'));
		Out.Append(PropertyValue.ReplaceCharWithEscapedChar());
		Out.AppendChar(TEXT('"'));
	}
	else
	{
		Out.Append(PropertyValue);
	}

	static const int32 LineTerminatorLen = FCString::Strlen(LINE_TERMINATOR);
	Out.Append(LINE_TERMINATOR, LineTerminatorLen);
}

/**
* Looks for any overrides on the commandline for this file
*
* @param File Config to possibly modify
* @param Filename Name of the .ini file to look for overrides
*/
void FConfigFile::OverrideFromCommandline(FConfigCommandStream* File, const FString& Filename)
{
#if ALLOW_INI_OVERRIDE_FROM_COMMANDLINE

	// look for this filename on the commandline in the format:
//		-ini:IniName:[Section1]:Key=Value
// for example:
//		-ini:Engine:[/Script/Engine.Engine]:bSmoothFrameRate=False
//			(will update the cache after the final combined engine.ini)

	TStringBuilder<260> IniSwitchStringBuilder;
	IniSwitchStringBuilder.Append(CommandlineOverrideSpecifiers::IniSwitchIdentifier);
	IniSwitchStringBuilder.Append(FPathViews::GetBaseFilename(Filename));
	IniSwitchStringBuilder.Append(TEXT(":")); // Ensure we only match the exact filename

	// Initial search to early out if the -ini:IniName: pattern doesn't exist anywhere in the string
	// Cannot use find result directly as text can be found inside another argument
	if (FCString::Strifind(FCommandLine::Get(), *IniSwitchStringBuilder, true) == nullptr)
	{
		return;
	}

	// Null terminate
	const FStringView IniSwitch = *IniSwitchStringBuilder;
	const TCHAR* RemainingCommandLineStream = FCommandLine::Get();

	FString NextCommandLineArgumentToken;
	while (FParse::Token(RemainingCommandLineStream, NextCommandLineArgumentToken, /*bUseEscape=*/false))
	{
		if (NextCommandLineArgumentToken.StartsWith(IniSwitch))
		{
			FString SettingsString = NextCommandLineArgumentToken.RightChop(IniSwitch.Len());

			// break apart on the commas. WARNING: This is supported for legacy reasons only
			// Providing multiple key-value pairs in a single -ini argument breaks when combined 
			// with quoted values. Fixing this is non-trivial and likely platform dependent.
			TArray<FString> SettingPairs;
			{
				FString NextSettingToken;
				const TCHAR* SettingsStream = *SettingsString;
				while (FParse::Expression(SettingsStream, NextSettingToken, false, CommandlineOverrideSpecifiers::PropertySeperator))
				{
					SettingPairs.Add(MoveTemp(NextSettingToken));
				}
			}

			for (int32 Index = 0; Index < SettingPairs.Num(); Index++)
			{
				// set each one, by splitting on the =
				FString SectionAndKey, Value;
				if (SettingPairs[Index].Split(TEXT("="), &SectionAndKey, &Value))
				{
					// now we need to split off the key from the rest of the section name
					int32 SectionNameEndIndex = SectionAndKey.Find(CommandlineOverrideSpecifiers::PropertyStartIdentifier, ESearchCase::IgnoreCase, ESearchDir::FromEnd);
					// check for malformed string
					if (SectionNameEndIndex == INDEX_NONE || SectionNameEndIndex == 0)
					{
						continue;
					}

					FString Section = SectionAndKey.Left(SectionNameEndIndex);

					// Remove commandline syntax from the section name.
					Section = Section.Replace(CommandlineOverrideSpecifiers::IniNameEndIdentifier, TEXT(""));
					Section = Section.Replace(CommandlineOverrideSpecifiers::PropertyStartIdentifier, TEXT(""));
					Section = Section.Replace(CommandlineOverrideSpecifiers::SectionStartIdentifier, TEXT(""));

					FString PropertyKey = SectionAndKey.Mid(SectionNameEndIndex + UE_ARRAY_COUNT(CommandlineOverrideSpecifiers::PropertyStartIdentifier) - 1);

					// If the property value was quoted, remove the quotes
					if (Value.Len() > 1 && Value.StartsWith(TEXT("\"")) && Value.EndsWith(TEXT("\"")))
					{
						Value = Value.Mid(1, Value.Len() - 2);
					}

					FConfigValue::EValueType ValueType = FConfigValue::EValueType::Set;
					if (PropertyKey.StartsWith(TEXT("-")))
					{
						PropertyKey.RemoveFromStart(TEXT("-"));
						ValueType = FConfigValue::EValueType::Remove;
					}
					else if (PropertyKey.StartsWith(TEXT("+")))
					{
						PropertyKey.RemoveFromStart(TEXT("+"));
						ValueType = FConfigValue::EValueType::ArrayAdd;
					}

					FConfigCommandStreamSection* SectionToModify = File->FindOrAddSectionInternal(Section);
					SectionToModify->Emplace(*PropertyKey, FConfigValue(MoveTemp(Value), ValueType));
				}
			}
		}
	}
#endif
}

void FConfigFile::OverrideFromCommandline(FConfigFile* File, const FString& Filename)
{
#if ALLOW_INI_OVERRIDE_FROM_COMMANDLINE

	// this would already be handled with new dynamic layer stuff
	static bool bUseNewDynamicLayers = IConsoleManager::Get().FindConsoleVariable(TEXT("ini.UseNewDynamicLayers"))->GetInt() != 0;
	if (bUseNewDynamicLayers)
	{
		return;
	}

	// look for this filename on the commandline in the format:
	//		-ini:IniName:[Section1]:Key=Value
	// for example:
	//		-ini:Engine:[/Script/Engine.Engine]:bSmoothFrameRate=False
	//			(will update the cache after the final combined engine.ini)

	TStringBuilder<260> IniSwitchStringBuilder;
	IniSwitchStringBuilder.Append(CommandlineOverrideSpecifiers::IniSwitchIdentifier);
	IniSwitchStringBuilder.Append(FPathViews::GetBaseFilename(Filename));
	IniSwitchStringBuilder.Append(TEXT(":")); // Ensure we only match the exact filename

	// Initial search to early out if the -ini:IniName: pattern doesn't exist anywhere in the string
	// Cannot use find result directly as text can be found inside another argument
	if (FCString::Strifind(FCommandLine::Get(), *IniSwitchStringBuilder, true) == nullptr)
	{
		return;
	}

	// Null terminate
	const FStringView IniSwitch = *IniSwitchStringBuilder;
	const TCHAR* RemainingCommandLineStream = FCommandLine::Get();

	FString NextCommandLineArgumentToken;
	while(FParse::Token(RemainingCommandLineStream, NextCommandLineArgumentToken, /*bUseEscape=*/false))
	{
		if (NextCommandLineArgumentToken.StartsWith(IniSwitch))
		{
			FString SettingsString = NextCommandLineArgumentToken.RightChop(IniSwitch.Len());

			// break apart on the commas. WARNING: This is supported for legacy reasons only
			// Providing multiple key-value pairs in a single -ini argument breaks when combined 
			// with quoted values. Fixing this is non-trivial and likely platform dependent.
			TArray<FString> SettingPairs;
			{
				FString NextSettingToken;
				const TCHAR* SettingsStream = *SettingsString;
				while (FParse::Token(SettingsStream, NextSettingToken, false, CommandlineOverrideSpecifiers::PropertySeperator))
				{
					SettingPairs.Add(MoveTemp(NextSettingToken));
				}
			}

			for (int32 Index = 0; Index < SettingPairs.Num(); Index++)
			{
				// set each one, by splitting on the =
				FString SectionAndKey, Value;
				if (SettingPairs[Index].Split(TEXT("="), &SectionAndKey, &Value))
				{
					// now we need to split off the key from the rest of the section name
					int32 SectionNameEndIndex = SectionAndKey.Find(CommandlineOverrideSpecifiers::PropertyStartIdentifier, ESearchCase::IgnoreCase, ESearchDir::FromEnd);
					// check for malformed string
					if (SectionNameEndIndex == INDEX_NONE || SectionNameEndIndex == 0)
					{
						continue;
					}

					// Create the commandline override object
					FConfigCommandlineOverride& CommandlineOption = File->CommandlineOptions[File->CommandlineOptions.Emplace()];
					CommandlineOption.BaseFileName = *FPaths::GetBaseFilename(Filename);
					CommandlineOption.Section = SectionAndKey.Left(SectionNameEndIndex);
					
					// Remove commandline syntax from the section name.
					CommandlineOption.Section = CommandlineOption.Section.Replace(CommandlineOverrideSpecifiers::IniNameEndIdentifier, TEXT(""));
					CommandlineOption.Section = CommandlineOption.Section.Replace(CommandlineOverrideSpecifiers::PropertyStartIdentifier, TEXT(""));
					CommandlineOption.Section = CommandlineOption.Section.Replace(CommandlineOverrideSpecifiers::SectionStartIdentifier, TEXT(""));

					CommandlineOption.PropertyKey = SectionAndKey.Mid(SectionNameEndIndex + UE_ARRAY_COUNT(CommandlineOverrideSpecifiers::PropertyStartIdentifier) - 1);
					
					// If the property value was quoted, remove the quotes
					if (Value.Len() > 1 && Value.StartsWith(TEXT("\"")) && Value.EndsWith(TEXT("\"")))
					{
						CommandlineOption.PropertyValue = Value.Mid(1, Value.Len() - 2);
					}
					else
					{
						CommandlineOption.PropertyValue = Value;
					}

					// now put it into this into the cache
					if (CommandlineOption.PropertyKey.StartsWith(TEXT("-")))
					{
						CommandlineOption.PropertyKey.RemoveFromStart(TEXT("-"));

						TArray<FString> ValueArray;
						File->GetArray(*CommandlineOption.Section, *CommandlineOption.PropertyKey, ValueArray);
						ValueArray.Remove(CommandlineOption.PropertyValue);
						File->SetArray(*CommandlineOption.Section, *CommandlineOption.PropertyKey, ValueArray);
					}
					else if (CommandlineOption.PropertyKey.StartsWith(TEXT("+")))
					{
						CommandlineOption.PropertyKey.RemoveFromStart(TEXT("+"));

						TArray<FString> ValueArray;
						File->GetArray(*CommandlineOption.Section, *CommandlineOption.PropertyKey, ValueArray);
						ValueArray.Add(CommandlineOption.PropertyValue);
						File->SetArray(*CommandlineOption.Section, *CommandlineOption.PropertyKey, ValueArray);
					}
					else
					{
						File->SetString(*CommandlineOption.Section, *CommandlineOption.PropertyKey, *CommandlineOption.PropertyValue);
					}
				}	
			}
		}
	}
#endif
}


namespace UE::ConfigCacheIni::Private
{

struct FImpl
{
/**
 * Check if the provided config section has a property which matches the one we are providing
 *
 * @param InSection			- The section which we are looking for a match in
 * @param InPropertyName	- The name of the property we are looking to match
 * @param InPropertyValue	- The value of the property which, if found, we are checking a match
 *
 * @return True if a property was found in the InSection which matched the Property Name and Value.
 */
static bool DoesConfigPropertyValueMatch(const FConfigSection* InSection, const FName& InPropertyName, const FString& InPropertyValue)
{
	bool bFoundAMatch = false;

	if (InSection)
	{
		const bool bIsInputStringValidFloat = FDefaultValueHelper::IsStringValidFloat(InPropertyValue);

		// Start Array check, if the property is in an array, we need to iterate over all properties.
		for (FConfigSection::TConstKeyIterator It(*InSection, InPropertyName); It && !bFoundAMatch; ++It)
		{
			const FString& PropertyValue = It.Value().GetSavedValueForWriting();
			bFoundAMatch =
				PropertyValue.Len() == InPropertyValue.Len() &&
				PropertyValue == InPropertyValue;

			// if our properties don't match, run further checks
			if (!bFoundAMatch)
			{
				// Check that the mismatch isn't just a string comparison issue with floats
				if (bIsInputStringValidFloat && FDefaultValueHelper::IsStringValidFloat(PropertyValue))
				{
					bFoundAMatch = FCString::Atof(*PropertyValue) == FCString::Atof(*InPropertyValue);
				}
			}
		}
	}


	return bFoundAMatch;
}

}; // struct FImpl

} // namespace UE::ConfigCacheIni::Private

/**
 * Check if the provided property information was set as a commandline override
 *
 * @param InConfigFile		- The config file which we want to check had overridden values
 * @param InSectionName		- The name of the section which we are checking for a match
 * @param InPropertyName		- The name of the property which we are checking for a match
 * @param InPropertyValue	- The value of the property which we are checking for a match
 *
 * @return True if a commandline option was set that matches the input parameters
 */
bool PropertySetFromCommandlineOption(const FConfigFile* InConfigFile, const FString& InSectionName, const FName& InPropertyName, const FString& InPropertyValue)
{
	FString Value;
	if (InConfigFile->Branch != nullptr)
	{
		FConfigCommandStreamSection* Section = InConfigFile->Branch->CommandLineOverrides.Find(InSectionName);
		if (Section != nullptr)
		{
			return Section->FindPair(InPropertyName, InPropertyValue) != nullptr;
		}
	}
	
	return false;
}

bool FConfigFile::WriteTempFileThenMove()
{
#if PLATFORM_DESKTOP && WITH_EDITOR
	bool bWriteTempFileThenMove = !FApp::IsGame() && !FApp::IsUnattended();
#else // PLATFORM_DESKTOP
	bool bWriteTempFileThenMove = false;
#endif

	return bWriteTempFileThenMove;
}

bool FConfigFile::Write(const FString& Filename, bool bDoRemoteWrite/* = true*/, const FString& PrefixText/*=FString()*/)
{
	TMap<FString, FString> SectionTexts;
	TArray<FString> SectionOrder;
	if (!PrefixText.IsEmpty())
	{
		SectionTexts.Add(FString(), PrefixText);
	}
	return WriteInternal(Filename, bDoRemoteWrite, SectionTexts, SectionOrder, nullptr);
}

void FConfigFile::WriteToString(FString& InOutText, const FString& SimulatedFilename /*= FString()*/, const FString& PrefixText /*= FString()*/)
{
	TMap<FString, FString> SectionTexts;
	TArray<FString> SectionOrder;
	if (!PrefixText.IsEmpty())
	{
		SectionTexts.Add(FString(), PrefixText);
	}

	int32 IniCombineThreshold = MAX_int32;
	bool bIsADefaultIniWrite = IsADefaultIniWrite(SimulatedFilename, IniCombineThreshold);

	WriteToStringInternal(InOutText, bIsADefaultIniWrite, IniCombineThreshold, SectionTexts, SectionOrder);
}

bool FConfigFile::IsADefaultIniWrite(const FString& Filename, int32& OutIniCombineThreshold) const
{
	bool bIsADefaultIniWrite = Branch != nullptr && Filename != Branch->IniPath;

	OutIniCombineThreshold = MAX_int32;
	if (bIsADefaultIniWrite)
	{
		// find the filename in ini hierarchy
		FString IniName = FPaths::GetCleanFilename(Filename);
		for (const auto& HierarchyFileIt : Branch->Hierarchy)
		{
			if (FPaths::GetCleanFilenameUtf8(HierarchyFileIt.Value) == IniName)
			{
				OutIniCombineThreshold = HierarchyFileIt.Key;
				break;
			}
		}
	}

	return bIsADefaultIniWrite;
}

bool FConfigFile::WriteInternal(const FString& Filename, bool bDoRemoteWrite, TMap<FString, FString>& InOutSectionTexts, const TArray<FString>& InSectionOrder, FString* OriginalContents)
{
	int32 IniCombineThreshold = MAX_int32;
	bool bIsADefaultIniWrite = IsADefaultIniWrite(Filename, IniCombineThreshold);

	FString Text;
	WriteToStringInternal(Text, bIsADefaultIniWrite, IniCombineThreshold, InOutSectionTexts, InSectionOrder);

	// don't write out non-default configs that are only whitespace
	if (!bIsADefaultIniWrite && Text.TrimStart().Len() == 0)
	{
		DeleteConfigFileWrapper(*Filename);
		return true;
	}
	
	if (!Dirty || NoSave || !AreWritesAllowedGlobally())
	{
		return true;
	}

	if (bDoRemoteWrite)
	{
		// Write out the remote version (assuming it was loaded)
		FRemoteConfig::Get()->Write(*Filename, Text);
	}

	// don't write anything out if it didn't actually change
	if (OriginalContents != nullptr && Text == *OriginalContents)
	{
		Dirty = false;
		return true;
	}

	bool bResult = SaveConfigFileWrapper(*Filename, Text);

	// File is still dirty if it didn't save.
	Dirty = !bResult;

	// Return if the write was successful
	return bResult;
}

void FConfigFile::WriteToStringInternal(FString& InOutText, bool bIsADefaultIniWrite, int32 IniCombineThreshold, TMap<FString, FString>& InOutSectionTexts, const TArray<FString>& InSectionOrder)
{
	const int32 InitialInOutTextSize = InOutText.Len();

	// Estimate max size to reduce re-allocations (does not inspect actual properties for performance)
	int32 InitialEstimatedFinalTextSize = 0;
	int32 HighestPropertiesInSection = 0;
	for (TIterator SectionIterator(*this); SectionIterator; ++SectionIterator)
	{
		HighestPropertiesInSection = FMath::Max(HighestPropertiesInSection, SectionIterator.Value().Num());
		InitialEstimatedFinalTextSize += ((SectionIterator.Value().Num() + 1) * 90);
	}
	// Limit size estimate to avoid pre-allocating too much memory
	InitialEstimatedFinalTextSize = FMath::Min(InitialEstimatedFinalTextSize, 128 * 1024 * 1024);
	InOutText.Reserve(InitialInOutTextSize + InitialEstimatedFinalTextSize);

	TArray<FString> SectionOrder;
	SectionOrder.Reserve(InSectionOrder.Num() + this->Num());
	SectionOrder.Append(InSectionOrder);
	InOutSectionTexts.Reserve(InSectionOrder.Num() + this->Num());

	TArray<const FConfigValue*> CompletePropertyToWrite;
	FString PropertyNameString;
	TSet<FName> PropertiesAddedLookup;
	PropertiesAddedLookup.Reserve(HighestPropertiesInSection);
	int32 EstimatedFinalTextSize = 0;
	
	// no need to look up the section if it's a default ini, or if we are always saving all sections
	const FConfigSection* SectionsToSaveSection = (bIsADefaultIniWrite || bCanSaveAllSections) ? nullptr : FindSection(SectionsToSaveStr);
	TArray<FString> SectionsToSave;
	if (SectionsToSaveSection != nullptr)
	{
		// Do not report the read of SectionsToSave. Some ConfigFiles are reallocated without it, and we
		// log that the section disappeared. But this log is spurious since the only reason it was read was
		// for the internal save before the FConfigFile is made publicly available.
		TArray<const FConfigValue*, TInlineAllocator<10>> SectionsToSaveValues;
		SectionsToSaveSection->MultiFindPointer("Section", SectionsToSaveValues);
		SectionsToSave.Reserve(SectionsToSaveValues.Num());
		for (const FConfigValue* ConfigValue : SectionsToSaveValues)
		{
			SectionsToSave.Add(ConfigValue->GetValueForWriting());
		}
	}
	
	for( TIterator SectionIterator(*this); SectionIterator; ++SectionIterator )
	{
		const FString& SectionName = SectionIterator.Key();
		const FConfigSection& Section = SectionIterator.Value();
		
		// null Sections array means to save everything, otherwise check if we can save this section
		bool bCanSaveThisSection = SectionsToSaveSection == nullptr || SectionsToSave.Contains(SectionName);
		if (!bCanSaveThisSection)
		{
			continue;
		}

		// If we have a config file to check against, have a look.
		const FConfigSection* SourceConfigSection = nullptr;
		if (Branch != nullptr && Branch->FinalCombinedLayers.Num() > 0)
		{
			// Check the sections which could match our desired section name
			SourceConfigSection = Branch->FinalCombinedLayers.FindSection(SectionName);

#if !UE_BUILD_SHIPPING
			if (!SourceConfigSection && FPlatformProperties::RequiresCookedData() == false && SectionName.StartsWith(TEXT("/Script/")))
			{
				// Guard against short names in ini files
				const FString ShortSectionName = SectionName.Replace(TEXT("/Script/"), TEXT(""));
				if (Branch->FinalCombinedLayers.FindSection(ShortSectionName) != nullptr)
				{
					UE_LOGF(LogConfig, Fatal, "Short config section found while looking for %ls", *SectionName);
				}
			}
#endif
		}

		InOutText.LeftInline(InitialInOutTextSize, EAllowShrinking::No);
		PropertiesAddedLookup.Reset();

		for( FConfigSection::TConstIterator It2(Section); It2; ++It2 )
		{
			const FName PropertyName = It2.Key();
			// Use GetSavedValueForWriting rather than GetSavedValue to avoid having this save operation mark the values as having been accessed for dependency tracking
			const FString& PropertyValue = It2.Value().GetSavedValueForWriting();

			// Check if the we've already processed a property of this name. If it was part of an array we may have already written it out.
			if( !PropertiesAddedLookup.Contains( PropertyName ) )
			{
				// check whether the option we are attempting to write out, came from the commandline as a temporary override.
				const bool bOptionIsFromCommandline = PropertySetFromCommandlineOption(this, SectionName, PropertyName, PropertyValue);

				// We ALWAYS want to write CurrentIniVersion.
				const bool bIsCurrentIniVersion = SectionName == CurrentIniVersionStr && PropertyName == VersionSectionName;

				// Check if the property matches the source configs. We do not wanna write it out if so.
				if ((bIsADefaultIniWrite || bIsCurrentIniVersion ||
					!UE::ConfigCacheIni::Private::FImpl::DoesConfigPropertyValueMatch(SourceConfigSection, PropertyName, PropertyValue))
					&& !bOptionIsFromCommandline)
				{
					// If this is the first property we are writing of this section, then print the section name
					if( InOutText.Len() == InitialInOutTextSize )
					{
						InOutText.Appendf(TEXT("[%s]" LINE_TERMINATOR_ANSI), *SectionName);

						// and if the section has any array of struct uniqueness keys, add them here
						for (auto It = Section.ArrayOfStructKeys.CreateConstIterator(); It; ++It)
						{
							InOutText.Appendf(TEXT("@%s=%s" LINE_TERMINATOR_ANSI), *It.Key().ToString(), *It.Value());
						}
					}

					// Write out our property, if it is an array we need to write out the entire array.
					CompletePropertyToWrite.Reset();
					Section.MultiFindPointer( PropertyName, CompletePropertyToWrite, true );

					if( bIsADefaultIniWrite )
					{
						ProcessPropertyAndWriteForDefaults(IniCombineThreshold, CompletePropertyToWrite, InOutText, SectionName, PropertyName.ToString());
					}
					else
					{
						PropertyNameString.Reset(FName::StringBufferSize);
						PropertyName.AppendString(PropertyNameString);
						for (const FConfigValue* ConfigValue : CompletePropertyToWrite)
						{
							// Use GetSavedValueForWriting rather than GetSavedValue to avoid marking these values used during save to disk as having been accessed for dependency tracking
							AppendExportedPropertyLine(InOutText, PropertyNameString, ConfigValue->GetSavedValueForWriting());
						}
					}

					PropertiesAddedLookup.Add( PropertyName );
				}
			}
		}

		// If we didn't decide to write any properties on this section, then we don't add the section
		// to the destination file
		if (InOutText.Len() > InitialInOutTextSize)
		{
			InOutSectionTexts.FindOrAdd(SectionName) = InOutText.RightChop(InitialInOutTextSize);

			// Add the Section to SectionOrder in case it's not already there
			SectionOrder.Add(SectionName);

			EstimatedFinalTextSize += InOutText.Len() - InitialInOutTextSize + 4;
		}
		else
		{
			InOutSectionTexts.Remove(SectionName);
		}
	}

	// Join all of the sections together
	InOutText.LeftInline(InitialInOutTextSize, EAllowShrinking::No);
	InOutText.Reserve(InitialInOutTextSize + EstimatedFinalTextSize);
	TSet<FString> SectionNamesLeftToWrite;
	SectionNamesLeftToWrite.Reserve(InOutSectionTexts.Num());
	for (TPair<FString,FString>& kvpair : InOutSectionTexts)
	{
		SectionNamesLeftToWrite.Add(kvpair.Key);
	}

	static const FString BlankLine(TEXT(LINE_TERMINATOR_ANSI LINE_TERMINATOR_ANSI));
	auto AddSectionToText = [&InOutText, &InOutSectionTexts, &SectionNamesLeftToWrite](const FString& SectionName)
	{
		FString* SectionText = InOutSectionTexts.Find(SectionName);
		if (!SectionText)
		{
			return;
		}
		if (SectionNamesLeftToWrite.Remove(SectionName) == 0)
		{
			// We already wrote this section
			return;
		}
		InOutText.Append(*SectionText);
		if (!InOutText.EndsWith(BlankLine, ESearchCase::CaseSensitive))
		{
			InOutText.Append(LINE_TERMINATOR);
		}
	};

	// First add the empty section
	AddSectionToText(FString());

	// Second add all the sections in SectionOrder; this includes any sections in *this that were not in InSectionOrder, because we added them during the loop
	for (FString& SectionName : SectionOrder)
	{
		AddSectionToText(SectionName);
	}

	// Third add any remaining sections that were passed in in InOutSectionTexts but were not specified in InSectionOrder and were not in *this
	if (SectionNamesLeftToWrite.Num() > 0)
	{
		TArray<FString> RemainingNames;
		RemainingNames.Reserve(SectionNamesLeftToWrite.Num());
		for (FString& SectionName : SectionNamesLeftToWrite)
		{
			RemainingNames.Add(SectionName);
		}
		RemainingNames.Sort();
		for (FString& SectionName : RemainingNames)
		{
			AddSectionToText(SectionName);
		}
	}
}

/** Adds any properties that exist in InSourceFile that this config file is missing */
void FConfigFile::AddMissingProperties( const FConfigFile& InSourceFile )
{
	for( TConstIterator SourceSectionIt( InSourceFile ); SourceSectionIt; ++SourceSectionIt )
	{
		const FString& SourceSectionName = SourceSectionIt.Key();
		const FConfigSection& SourceSection = SourceSectionIt.Value();

		{
			// If we don't already have this section, go ahead and add it now
			FConfigSection* DestSection = FindOrAddSectionInternal( SourceSectionName );
			DestSection->Reserve(SourceSection.Num());

			for( FConfigSection::TConstIterator SourcePropertyIt( SourceSection ); SourcePropertyIt; ++SourcePropertyIt )
			{
				const FName SourcePropertyName = SourcePropertyIt.Key();
				
				// If we don't already have this property, go ahead and add it now
				if( DestSection->Find( SourcePropertyName ) == nullptr )
				{
					TArray<const FConfigValue*, TInlineAllocator<32>> Results;
					SourceSection.MultiFindPointer(SourcePropertyName, Results, true);
					for (const FConfigValue* Result : Results)
					{
						FConfigValue& AddedValue = DestSection->Add(SourcePropertyName, *Result);
#if UE_WITH_CONFIG_TRACKING 
						AddedValue.SetSectionAccess(DestSection->SectionAccess.GetReference());
#endif
						Dirty = true;
					}
				}
			}
		}
	}
}



void FConfigFile::Dump(FOutputDevice& Ar)
{
	Ar.Logf( TEXT("FConfigFile::Dump") );

	for( TMap<FString,FConfigSection>::TIterator It(*this); It; ++It )
	{
		Ar.Logf( TEXT("[%s]"), *It.Key() );
		TArray<FName> KeyNames;

		FConfigSection& Section = It.Value();
		Section.GetKeys(KeyNames);
		for(TArray<FName>::TConstIterator KeyNameIt(KeyNames);KeyNameIt;++KeyNameIt)
		{
			const FName KeyName = *KeyNameIt;

			TArray<FConfigValue> Values;
			Section.MultiFind(KeyName,Values,true);

			if ( Values.Num() > 1 )
			{
				for ( int32 ValueIndex = 0; ValueIndex < Values.Num(); ValueIndex++ )
				{
					Ar.Logf(TEXT("	%s[%i]=%s"), *KeyName.ToString(), ValueIndex, *Values[ValueIndex].GetValue().ReplaceCharWithEscapedChar());
				}
			}
			else
			{
				Ar.Logf(TEXT("	%s=%s"), *KeyName.ToString(), *Values[0].GetValue().ReplaceCharWithEscapedChar());
			}
		}

		Ar.Log( LINE_TERMINATOR );
	}
}

int32 FConfigFile::GetArray(const TCHAR* Section, const TCHAR* Key, TArray<FString>& Value) const
{
	if (const FConfigSection* ConfigSection = FindSection(Section))
	{
		return ConfigSection->GetArray(Key, Value);
	}
#if !UE_BUILD_SHIPPING
	CheckLongSectionNames(Section, this);
#endif
	return false;
}

bool FConfigFile::DoesSectionExist(const TCHAR* Section) const
{
	return FindSection(Section) != nullptr;
}

void FConfigFile::SetString( const TCHAR* Section, const TCHAR* Key, const TCHAR* Value )
{
	FConfigSection* Sec = FindOrAddSectionInternal( Section );

	FConfigValue* ConfigValue = Sec->Find( Key );
	if( ConfigValue == nullptr )
	{
		Sec->Add(Key, FConfigValue(Sec, FName(Key), Value));
		Dirty = true;
	}
	// Use GetSavedValueForWriting rather than GetSavedValue to avoid reporting the is-it-dirty query mark the values as having been accessed for dependency tracking
	else if( FCString::Strcmp(*ConfigValue->GetSavedValueForWriting(),Value)!=0 )
	{
		Dirty = true;
		*ConfigValue = Value;
	}
}

void FConfigFile::SetText( const TCHAR* Section, const TCHAR* Key, const FText& Value )
{
	FConfigSection* Sec = FindOrAddSectionInternal( Section );

	FString StrValue;
	FTextStringHelper::WriteToBuffer(StrValue, Value);

	FConfigValue* ConfigValue = Sec->Find( Key );
	if( ConfigValue == nullptr )
	{
		Sec->Add(Key, FConfigValue(Sec, FName(Key), MoveTemp(StrValue)));
		Dirty = true;
	}
	// Use GetSavedValueForWriting rather than GetSavedValue to avoid reporting the is-it-dirty query mark the values as having been accessed for dependency tracking
	else if( FCString::Strcmp(*ConfigValue->GetSavedValueForWriting(), *StrValue)!=0 )
	{
		Dirty = true;
		*ConfigValue = MoveTemp(StrValue);
	}
}

void FConfigFile::SetFloat(const TCHAR* Section, const TCHAR* Key, float Value)
{
	TCHAR Text[MAX_SPRINTF];
	FCString::Sprintf(Text, TEXT("%.*g"), std::numeric_limits<float>::max_digits10, Value);
	SetString(Section, Key, Text);
}

void FConfigFile::SetDouble(const TCHAR* Section, const TCHAR* Key, double Value)
{
	TCHAR Text[MAX_SPRINTF];
	FCString::Sprintf(Text, TEXT("%.*g"), std::numeric_limits<double>::max_digits10, Value);
	SetString(Section, Key, Text);
}

void FConfigFile::SetBool(const TCHAR* Section, const TCHAR* Key, bool Value)
{
	SetString(Section, Key, Value ? TEXT("True") : TEXT("False"));
}

void FConfigFile::SetInt64( const TCHAR* Section, const TCHAR* Key, int64 Value )
{
	TCHAR Text[MAX_SPRINTF];
	FCString::Sprintf( Text, TEXT("%lld"), Value );
	SetString( Section, Key, Text );
}


void FConfigFile::SetArray(const TCHAR* SectionName, const TCHAR* Key, const TArray<FString>& Value)
{
	FConfigSection* Section = FindOrAddSectionInternal(SectionName);

	if (Section->Remove(Key) > 0)
	{
		Dirty = true;
	}

	for (int32 i = 0; i < Value.Num(); i++)
	{
		Section->Add(Key, FConfigValue(Section, FName(Key), Value[i]));
		Dirty = true;
	}
	
	if (ChangeTracker)
	{
		// remove anything to do with this array in the tracker
		FConfigCommandStreamSection* Sec = ChangeTracker->FindOrAddSectionInternal(SectionName);
		
		// if there were any entries to remove, then add a clear operation
		if (Sec->Remove(Key) > 0)
		{
			Sec->Add(Key, FConfigValue(TEXT("__Clear__"), FConfigValue::EValueType::Clear));
		}
		// then add all entries
		for (int32 i = 0; i < Value.Num(); i++)
		{
			Sec->Add(Key, FConfigValue(Value[i], FConfigValue::EValueType::ArrayAdd));
		}
	}
}

bool FConfigFile::SetInSection(const TCHAR* SectionName, FName Key, const FString& Value)
{
	FConfigSection* Section = FindOrAddSectionInternal(SectionName);
	Section->Remove(Key);
	Section->Add(Key, FConfigValue(Section, Key, Value));
	Dirty = true;

	if (ChangeTracker != nullptr)
	{
		FConfigCommandStreamSection* Sec = ChangeTracker->FindOrAddSectionInternal(SectionName);
		Sec->Remove(Key);
		Sec->Add(Key, FConfigValue(Value, FConfigValue::EValueType::Set));
	}

	return true;
}

bool FConfigFile::AddToSection(const TCHAR* SectionName, FName Key, const FString& Value)
{
	FConfigSection* Section = FindOrAddSectionInternal(SectionName);
	Section->Add(Key, FConfigValue(Section, Key, Value));
	Dirty = true;
	
	if (ChangeTracker != nullptr)
	{
		FConfigCommandStreamSection* Sec = ChangeTracker->FindOrAddSectionInternal(SectionName);
		Sec->Add(Key, FConfigValue(Value, FConfigValue::EValueType::ArrayAdd));
	}
	
	return true;
}

bool FConfigFile::AddUniqueToSection(const TCHAR* SectionName, FName Key, const FString& Value)
{
	FConfigSection* Section = FindOrAddSectionInternal(SectionName);
	if (Section->FindPair(Key, FConfigValue(Section, Key, Value)))
	{
		return false;
	}
	
	// just call Add since we already checked above if it exists (AddUnique can't return whether or not it existed)
	Section->Add(Key, FConfigValue(Section, Key, Value));
	Dirty = true;
	
	if (ChangeTracker != nullptr)
	{
		FConfigCommandStreamSection* Sec = ChangeTracker->FindOrAddSectionInternal(SectionName);
		Sec->Add(Key, FConfigValue(Value, FConfigValue::EValueType::ArrayAddUnique));
	}

	return true;
}

bool FConfigFile::RemoveKeyFromSection(const TCHAR* SectionName, FName Key)
{
	FConfigSection* Section = FindInternal(SectionName);
	// if it doesn't contain the key for any number of values
	if (Section == nullptr || !Section->Contains(Key))
	{
		return false;
	}

	Section->Remove(Key);
	Dirty = true;
	
	if (ChangeTracker != nullptr)
	{
		FConfigCommandStreamSection* Sec = ChangeTracker->FindOrAddSectionInternal(SectionName);
		// remove any tracked changes for this key as they are all blown away now
		Sec->Remove(Key);
		Sec->Add(Key, FConfigValue(TEXT("__Clear__"), FConfigValue::EValueType::Clear));
	}

	return true;
}

bool FConfigFile::RemoveFromSection(const TCHAR* SectionName, FName Key, const FString& Value)
{
	FConfigSection* Section = FindInternal(SectionName);
	// if it doesn't contain the pair, do nothing
	if (Section == nullptr || !Section->FindPair(Key, FConfigValue(Section, Key, Value)))
	{
		return false;
	}

	// remove any copies of the pair
	Section->Remove(Key, FConfigValue(Section, Key, Value));
	Dirty = true;
	return true;
}

bool FConfigFile::ResetKeyInSection(const TCHAR* SectionName, FName Key)
{
	FConfigSection* Section = FindInternal(SectionName);
	// if it doesn't contain the key for any number of values
	if (Section == nullptr || !Section->Contains(Key))
	{
		return false;
	}

	Section->Remove(Key);

	if (ChangeTracker != nullptr)
	{
		// remove this key from being tracked, which is the difference between this and RemvoeKeyFromSection
		FConfigCommandStreamSection* Sec = ChangeTracker->FindOrAddSectionInternal(SectionName);
		Sec->Remove(Key);
	}
	
	Dirty = true;
	return true;
}

void FConfigFile::SaveSourceToBackupFile()
{
	FString Text;

	FString BetweenRunsDir = (FPaths::ProjectIntermediateDir() / TEXT("Config/CoalescedSourceConfigs/"));
	FString Filename = FString::Printf( TEXT( "%s%s.ini" ), *BetweenRunsDir, *Name.ToString() );

	for( TMap<FString,FConfigSection>::TIterator SectionIterator(Branch->FinalCombinedLayers); SectionIterator; ++SectionIterator )
	{
		const FString& SectionName = SectionIterator.Key();
		const FConfigSection& Section = SectionIterator.Value();

		Text += FString::Printf( TEXT("[%s]" LINE_TERMINATOR_ANSI), *SectionName);

		for( FConfigSection::TConstIterator PropertyIterator(Section); PropertyIterator; ++PropertyIterator )
		{
			const FName PropertyName = PropertyIterator.Key();
			// Use GetSavedValueForWriting rather than GetSavedValue to avoid having this save operation mark the values as having been accessed for dependency tracking
			const FString& PropertyValue = PropertyIterator.Value().GetSavedValueForWriting();
			Text += FConfigFile::GenerateExportedPropertyLine(PropertyName.ToString(), PropertyValue);
		}
		Text += LINE_TERMINATOR;
	}

	if(!SaveConfigFileWrapper(*Filename, Text))
	{
		UE_LOGF(LogConfig, Warning, "Failed to saved backup for config[%ls]", *Filename);
	}
}


void FConfigFile::ProcessSourceAndCheckAgainstBackup()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ProcessSourceAndCheckAgainstBackup);

	if (!FPlatformProperties::RequiresCookedData())
	{
		FString BetweenRunsDir = (FPaths::ProjectIntermediateDir() / TEXT("Config/CoalescedSourceConfigs/"));
		FString BackupFilename = FString::Printf( TEXT( "%s%s.ini" ), *BetweenRunsDir, *Name.ToString() );

		FConfigFile BackupFile;
		ProcessIniContents(*BackupFilename, *BackupFilename, &BackupFile, false, false);

#if UE_WITH_CONFIG_TRACKING
		UE::ConfigAccessTracking::FFile* LocalFileAccess = GetFileAccess();
#endif
		for (TMap<FString,FConfigSection>::TIterator SectionIterator(Branch->FinalCombinedLayers); SectionIterator; ++SectionIterator )
		{
			const FString& SectionName = SectionIterator.Key();
			const FConfigSection& SourceSection = SectionIterator.Value();
			const FConfigSection* BackupSection = BackupFile.FindSection( SectionName );
			
			if (BackupSection && !FConfigSection::AreSectionsEqualForWriting(SourceSection, *BackupSection))
			{
				this->Remove( SectionName );
				FConfigSection& AddedSection = this->Add( SectionName, SourceSection );
#if UE_WITH_CONFIG_TRACKING
				UE::ConfigAccessTracking::FSection* SectionAccess = LocalFileAccess ?
					new UE::ConfigAccessTracking::FSection(*LocalFileAccess, FStringView(SectionName)) : nullptr;
				AddedSection.SectionAccess = SectionAccess;
				for (TPair<FName, FConfigValue>& Pair : AddedSection)
				{
					Pair.Value.SetSectionAccess(SectionAccess);
				}
#endif
			}
		}

		SaveSourceToBackupFile();
	}
}

static TArray<FString> GetSourceProperties(const FConfigFileHierarchy& SourceIniHierarchy, int IniCombineThreshold, const FString& SectionName, const FString& PropertyName)
{
	// Build a config file out of this default configs hierarchy.
	FConfigCacheIni Hierarchy(EConfigCacheType::Temporary);

	int32 HighestFileIndex = 0;
	TArray<int32> ExistingEntries;
	SourceIniHierarchy.GetKeys(ExistingEntries);
	for (const int32& NextEntry : ExistingEntries)
	{
		HighestFileIndex = NextEntry > HighestFileIndex ? NextEntry : HighestFileIndex;
	}

	const FString LastFileInHierarchy = FString(SourceIniHierarchy.FindChecked(HighestFileIndex));
	FConfigFile& DefaultConfigFile = Hierarchy.Add(LastFileInHierarchy, FConfigFile());

	for (const auto& HierarchyFileIt : SourceIniHierarchy)
	{
		// Combine everything up to the level we're writing, but not including it.
		// Inclusion would result in a bad feedback loop where on subsequent writes
		// we would be diffing against the same config we've just written to.
		if (HierarchyFileIt.Key < IniCombineThreshold)
		{
			DefaultConfigFile.Combine(FString(HierarchyFileIt.Value));
		}
	}

	// Remove any array elements from the default configs hierearchy, we will add these in below
	// Note.	This compensates for an issue where strings in the hierarchy have a slightly different format
	//			to how the config system wishes to serialize them.
	TArray<FString> SourceArrayProperties;
	Hierarchy.GetArray(*SectionName, *PropertyName, SourceArrayProperties, LastFileInHierarchy);

	return SourceArrayProperties;
}

void FConfigFile::ProcessPropertyAndWriteForDefaults(int IniCombineThreshold, const TArray<const FConfigValue*>& InCompletePropertyToProcess, FString& OutText, const FString& SectionName, const FString& PropertyName)
{
	// Only process against a hierarchy if this config file has one.
	if (Branch->Hierarchy.Num() > 0)
	{
		FString CleanedPropertyName = PropertyName;
		const bool bHadPlus = CleanedPropertyName.RemoveFromStart(TEXT("+"));
		const bool bHadBang = CleanedPropertyName.RemoveFromStart(TEXT("!"));

		const FString PropertyNameWithRemoveOp = TEXT("-") + CleanedPropertyName;

		// look for pointless !Clear entries that the config system wrote out when it noticed the user didn't have any entries
		if (bHadBang && InCompletePropertyToProcess.Num() == 1 && InCompletePropertyToProcess[0]->GetSavedValue() == TEXT("__ClearArray__"))
		{
			const TArray<FString> SourceArrayProperties = GetSourceProperties(Branch->Hierarchy, IniCombineThreshold, SectionName, CleanedPropertyName);
			for (const FString& NextElement : SourceArrayProperties)
			{
				OutText.Append(GenerateExportedPropertyLine(PropertyNameWithRemoveOp, NextElement));
			}

			// We don't need to write out the ! entry so just leave now.
			return;
		}

		// Handle array elements from the configs hierarchy.
		if (bHadPlus || InCompletePropertyToProcess.Num() > 1)
		{
			const TArray<FString> SourceArrayProperties = GetSourceProperties(Branch->Hierarchy, IniCombineThreshold, SectionName, CleanedPropertyName);
			for (const FString& NextElement : SourceArrayProperties)
			{
				OutText.Append(GenerateExportedPropertyLine(PropertyNameWithRemoveOp, NextElement));
			}
		}
	}

	// Write the properties out to a file.
	for (const FConfigValue* PropertyIt : InCompletePropertyToProcess)
	{
		OutText.Append(GenerateExportedPropertyLine(PropertyName, PropertyIt->GetSavedValue()));
	}
}


/*-----------------------------------------------------------------------------
	FConfigCommandStream
-----------------------------------------------------------------------------*/


bool FConfigCommandStream::FillFileFromDisk(const FString& InFilename, bool bHandleSymbolCommands)
{
	return ::FillFileFromDisk(this, InFilename, bHandleSymbolCommands);
}

void FConfigCommandStream::ProcessCommand(SectionType* Section, FStringView SectionName, FConfigValue::EValueType Command, FName Key, FString&& Value)
{
	Section->Emplace(Key, FConfigValue(MoveTemp(Value), Command));
}

FConfigCommandStreamSection* FConfigCommandStream::FindOrAddSectionInternal(const FString& SectionName)
{
	return &FindOrAdd(SectionName);
}

void FConfigCommandStream::Shrink()
{
#if !UE_BUILD_SHIPPING
	extern double GConfigShrinkTime;
	if (IsInGameThread()) GConfigShrinkTime -= FPlatformTime::Seconds();
#endif

	TMap<FString, FConfigCommandStreamSection>::Shrink();
	for (auto& Pair : *this)
	{
		Pair.Value.Shrink();
	}

	PerObjectConfigArrayOfStructKeys.Shrink();
	for (auto& Pair : PerObjectConfigArrayOfStructKeys)
	{
		Pair.Value.Shrink();
	}

#if !UE_BUILD_SHIPPING
	if (IsInGameThread()) GConfigShrinkTime += FPlatformTime::Seconds();
#endif
}



/*-----------------------------------------------------------------------------
	FConfigBranch
-----------------------------------------------------------------------------*/

FConfigBranch::FConfigBranch()
	: bIsSafeUnloaded(false)
{
	static int DefaultReplayMethod = -1;
	if (DefaultReplayMethod == -1)
	{
		if (!FParse::Value(FCommandLine::Get(), TEXT("ConfigReplayMethod="), DefaultReplayMethod))
		{
			DefaultReplayMethod = GDefaultReplayMethod;
		}
	}
	
	switch (DefaultReplayMethod)
	{
		case 0:
			ReplayMethod = EBranchReplayMethod::NoReplay; break;
		case 1:
			ReplayMethod = EBranchReplayMethod::DynamicLayerReplay; break;
		case 2:
		default:
			ReplayMethod = EBranchReplayMethod::FullReplay; break;
	}

	InitFiles();
}

FConfigBranch::FConfigBranch(const FConfigFile& ExistingFile)
	: bIsSafeUnloaded(false)
	, bIsHierarchical(false)
	, InMemoryFile(ExistingFile)
{
	ReplayMethod = EBranchReplayMethod::NoReplay;
	
	InitFiles();
}

void FConfigBranch::InitFiles()
{
	SavedLayer.Branch = this;
	CombinedStaticLayers.Branch = this;
	FinalCombinedLayers.Branch = this;
	CommandLineOverrides.Branch = this;
	InMemoryFile.Branch = this;
	
	if (GUseNewSaveTracking)
	{
		InMemoryFile.ChangeTracker = &SavedLayer;
	}
}

void FConfigBranch::RunOnEachFile(TFunction<void(FConfigFile& File, const FString& Name)> Func)
{
	// cache the static layers so when remaking dynamic layers after removing a dynamic layer it's faster
	Func(CombinedStaticLayers, TEXT("CombinedStaticLayers"));
	Func(FinalCombinedLayers, TEXT("FinalCombinedLayers"));
	Func(InMemoryFile, TEXT("InMemoryFile"));
}

void FConfigBranch::RunOnEachCommandStream(TFunction<void(FConfigCommandStream& File, const FString& Name)> Func)
{
	for (TPair<FString, FConfigCommandStream>& Pair : StaticLayers)
	{
		Func(Pair.Value, Pair.Key);
	}

	for (DynamicLayerList::TIterator Node(DynamicLayers.GetHead()); Node; ++Node)
	{
		Func(*Node.GetNode()->GetValue(), Node->Filename);
	}

	Func(SavedLayer, TEXT("SavedLayer"));
	Func(CommandLineOverrides, TEXT("CommandLineOverrides"));
//	Func(RuntimeChanges, TEXT("RuntimeChanges"));
}


bool FConfigBranch::AddDynamicLayerToHierarchy(const FString& Filename, FConfigModificationTracker* ModificationTracker, const TSet<FString>* GlobalConfigFileCache, const TSet<FString>* PluginConfigFileCache)
{
	if (!DoesConfigFileExistWrapper(*Filename, nullptr, GlobalConfigFileCache, PluginConfigFileCache))
	{
		return false;
	}

	return AddDynamicLayersToHierarchy({ { Filename, NAME_None, (uint16)DynamicLayerPriority::Unknown } }, ModificationTracker, GlobalConfigFileCache, PluginConfigFileCache);
}

bool FConfigBranch::AddDynamicLayersToHierarchy(const TArray<FDynamicLayerInfo>& Layers, FConfigModificationTracker* ModificationTracker, const TSet<FString>* GlobalConfigFileCache, const TSet<FString>* PluginConfigFileCache, bool bForceFullDynamicLayerUpdate)
{
#if UE_THREAD_SAFETY_PROTECT_GETSTRING
	// don't allow a read while modifying it here
	UE::TUniqueLock FileWriteLock(InMemoryFile.SharedMutex);
#endif

	LLM_SCOPE(ELLMTag::ConfigSystem);
	if (bIsSafeUnloaded)
	{
		SafeReload();
	}
	
	static bool bDumpIniLoadInfo = FParse::Param(FCommandLine::Get(), TEXT("dumpiniloads"));

	bool bFoundAFile = false;
	bool bInsertedBeforeEnd = false;

	// calculate a patch so we don't lose in-memory changes
	FConfigCommandStream Patch;
	if (bForceFullDynamicLayerUpdate)
	{
		Patch = CalculateDiff(FinalCombinedLayers, InMemoryFile);
	}

	TArray<FConfigCommandStream*> AddedLayers;
	for (const FDynamicLayerInfo& Layer : Layers)
	{
		UE_CLOGF(bDumpIniLoadInfo, LogConfig, Display, "Looking for file: %ls", *Layer.Filename);

		UE_LOGF(LogConfig, Verbose, "Adding Dynamic layer %ls to Branch %ls", *Layer.Filename, *IniName.ToString());

		if (!DoesConfigFileExistWrapper(*Layer.Filename, nullptr, GlobalConfigFileCache, PluginConfigFileCache))
		{
			UE_LOGF(LogConfig, Verbose, "  .. doesn't exist!");
			continue;
		}

		bFoundAFile = true;
		UE_CLOGF(bDumpIniLoadInfo, LogConfig, Display, "   Found %ls!", *Layer.Filename);

		if (AddedLayers.Num() == 0 && !bForceFullDynamicLayerUpdate)
		{
			Patch = CalculateDiff(FinalCombinedLayers, InMemoryFile);
			UE_LOGF(LogConfig, Verbose, "  .. calculating diff on first file");
		}
		
		// make and read in the layer
		FConfigCommandStream* DynamicLayer = new FConfigCommandStream;
		FillFileFromDisk(DynamicLayer, Layer.Filename, true);
		DynamicLayer->Priority = Layer.Priority;
		DynamicLayer->Filename = Layer.Filename;
		DynamicLayer->Tag = Layer.Tag;

		// remember in local array, then figure out how to remember it permanently
		AddedLayers.Add(DynamicLayer);


		// if we aren't caching dynamic layers, then we need a temp layer
		if (ReplayMethod == EBranchReplayMethod::NoReplay)
		{
			UE_LOGF(LogConfig, Verbose, "  .. no replay, so just adding at end");
		}
		else
		{
			// find the first node with higher priority
			// @todo move this to a function
			bool bInserted = false;
			for (DynamicLayerList::TIterator Node(DynamicLayers.GetHead()); Node; ++Node)
			{
				if (Node->Priority > DynamicLayer->Priority)
				{
					UE_LOGF(LogConfig, Verbose, "  .. inserted in middle of dynamic layers");
					DynamicLayers.InsertNode(DynamicLayer, Node.GetNode());
					bInsertedBeforeEnd = true;
					bInserted = true;
					break;
				}
			}
			if (!bInserted)
			{
				UE_LOGF(LogConfig, Verbose, "  .. inserting at end of layers");
				DynamicLayers.AddTail(DynamicLayer);
			}
		}

		// track modified section names if desired
		if (ModificationTracker != nullptr)
		{
			if (ModificationTracker->bTrackModifiedSections)
			{
				UE_LOGF(LogConfig, Verbose, "  .. tracking sections:");
				for (const TPair<FString, FConfigCommandStreamSection>& Pair : *DynamicLayer)
				{
					TSet<FString>& ModifiedSections = ModificationTracker->ModifiedSectionsPerBranch.FindOrAdd(IniName);
					ModifiedSections.Add(Pair.Key);
					UE_LOGF(LogConfig, Verbose, "  .. .. %ls", *Pair.Key);
					if (FConfigModificationTracker::FCVarTracker* CVarTracker = ModificationTracker->CVars.Find(Pair.Key))
					{
						UE_LOGF(LogConfig, Verbose, "  .. .. .. tracking cvars");
						
						TMap<FName, FConfigSection>& PerIniCVarSections = CVarTracker->CVarEntriesPerBranchPerTag.FindOrAdd(Layer.Tag);
						FConfigSection& TrackedCVarSection = PerIniCVarSections.FindOrAdd(IniName);
						
						const FConfigSectionMap& ModifiedCVars = (const FConfigSectionMap&)Pair.Value;  
						for (const TPair<FName, FConfigValue>& CVarPair : ModifiedCVars)
						{
							UE_LOGF(LogConfig, Verbose, "  .. .. .. .. %ls = %ls", *CVarPair.Key.ToString(), *CVarPair.Value.GetSavedValue());
							TrackedCVarSection.Remove(CVarPair.Key);
							TrackedCVarSection.Add(CVarPair.Key, CVarPair.Value);
						}
					}
				}
			}
			if (ModificationTracker->bTrackLoadedFiles)
			{
				ModificationTracker->LoadedFiles.Add(Layer.Filename);
			}
		}
	}
	
	if (AddedLayers.Num() > 0 || bForceFullDynamicLayerUpdate)
	{
		checkf(CommandLineOverrides.Num() == 0 || ReplayMethod != EBranchReplayMethod::NoReplay,
			TEXT("The branch (%s) with commandline overrides must not have ReplayMethod of NoReplay!"), *IniName.ToString());

		// if all were added at the end (or there's no replay), we can just apply them without rewinding
		// since commandline needs to trump the layers, we can't use this optimization if there are overrides
		if (!bInsertedBeforeEnd && !bForceFullDynamicLayerUpdate && CommandLineOverrides.Num() == 0)
		{
			for (FConfigCommandStream* NewLayer : AddedLayers)
			{
				UE_LOGF(LogConfig, Verbose, "  .. reapplying layer with %d sections", NewLayer->Num());
				FinalCombinedLayers.ApplyFile(NewLayer);
				InMemoryFile.ApplyFile(NewLayer);
			}
		}
		else
		{
			// rebuild
			FinalCombinedLayers = CombinedStaticLayers;
			UE_LOGF(LogConfig, Verbose, "  .. reapplying all dynamic layers");
			for (DynamicLayerList::TIterator Node(DynamicLayers.GetHead()); Node; ++Node)
			{
				FinalCombinedLayers.ApplyFile(*Node);
			}
			FinalCombinedLayers.ApplyFile(&CommandLineOverrides);
			bool bOldSaveAllSections = InMemoryFile.bCanSaveAllSections;
			InMemoryFile = FinalCombinedLayers;
			InMemoryFile.bCanSaveAllSections = bOldSaveAllSections;
		}

		// re-apply the in-memory changes
		InMemoryFile.ApplyFile(&Patch);

		FinalCombinedLayers.Shrink();
		InMemoryFile.Shrink();
	}

	return bFoundAFile;
}

bool FConfigBranch::AddDynamicLayerStringToHierarchy(const FString& Filename, const FString& Contents, FName Tag, DynamicLayerPriority Priority, FConfigModificationTracker* ModificationTracker)
{
#if UE_THREAD_SAFETY_PROTECT_GETSTRING
	// don't allow a read while modifying it here
	UE::TUniqueLock FileWriteLock(InMemoryFile.SharedMutex);
#endif

	LLM_SCOPE(ELLMTag::ConfigSystem);
	if (bIsSafeUnloaded)
	{
		SafeReload();
	}

	bool bInsertedAtEnd = false;

	// calculate a patch so we don't lose in-memory changes
	FConfigCommandStream Patch = CalculateDiff(FinalCombinedLayers, InMemoryFile);

	FConfigCommandStream* DynamicLayer = nullptr;
	FConfigCommandStream LocalLayer;
	// if we aren't caching dynamic layers, then we need a temp layer
	if (ReplayMethod == EBranchReplayMethod::NoReplay)
	{
		DynamicLayer = &LocalLayer;
		bInsertedAtEnd = true;
	}
	else
	{
		// find the first node with higher priority
		bool bInserted = false;
		DynamicLayer = new FConfigCommandStream;
		DynamicLayer->Priority = (uint16)Priority;
		DynamicLayer->Filename = Filename;
		for (DynamicLayerList::TIterator Node(DynamicLayers.GetHead()); Node; ++Node)
		{
			if (Node->Priority > DynamicLayer->Priority)
			{
				DynamicLayers.InsertNode(DynamicLayer, Node.GetNode());
				bInserted = true;
				break;
			}
		}
		if (!bInserted)
		{
			DynamicLayers.AddTail(DynamicLayer);
			bInsertedAtEnd = true;
		}
	}

	// we can't SafeUnload a string-based layer because we'll never be able to load it again
	DynamicLayer->bNeverSafeUnload = true;
	DynamicLayer->Tag = Tag;
	FillFileFromBuffer(DynamicLayer, Contents, true, Filename);

	// track modified section names if desired
	if (ModificationTracker != nullptr)
	{
		if (ModificationTracker->bTrackModifiedSections)
		{
			for (const TPair<FString, FConfigCommandStreamSection>& Pair : *DynamicLayer)
			{
				TSet<FString>& ModifiedSections = ModificationTracker->ModifiedSectionsPerBranch.FindOrAdd(IniName);
				ModifiedSections.Add(Pair.Key);
				if (FConfigModificationTracker::FCVarTracker* CVarTracker = ModificationTracker->CVars.Find(Pair.Key))
				{
					FConfigSection NewSection;
					// copy just the SectionMap parts
					(FConfigSectionMap&)NewSection = (const FConfigSectionMap&)Pair.Value;
					FConfigSection& SectionEntry = CVarTracker->CVarEntriesPerBranchPerTag.FindOrAdd(Tag).FindOrAdd(IniName);
					SectionEntry.Append(NewSection);
				}
			}
		}
		if (ModificationTracker->bTrackLoadedFiles)
		{
			ModificationTracker->LoadedFiles.Add(Filename);
		}
	}

	// if we inserted at the end, and there are no commandline options that need to stomp on layers,
	// then there's a chance of optimization
	if (!bInsertedAtEnd || CommandLineOverrides.Num() > 0)
	{
		// rebuild
		FinalCombinedLayers = CombinedStaticLayers;
		for (DynamicLayerList::TIterator Node(DynamicLayers.GetHead()); Node; ++Node)
		{
			FinalCombinedLayers.ApplyFile(*Node);
		}
		FinalCombinedLayers.ApplyFile(&CommandLineOverrides);
		bool bOldSaveAllSections = InMemoryFile.bCanSaveAllSections;
		InMemoryFile = FinalCombinedLayers;
		InMemoryFile.bCanSaveAllSections = bOldSaveAllSections;
	}
	else
	{
		FinalCombinedLayers.ApplyFile(DynamicLayer);
		InMemoryFile.ApplyFile(DynamicLayer);
	}

	// re-apply the in-memory changes
	InMemoryFile.ApplyFile(&Patch);

	FinalCombinedLayers.Shrink();
	InMemoryFile.Shrink();

	return true;
}


bool FConfigBranch::RemoveDynamicLayerFromHierarchy(const FString& Filename, FConfigModificationTracker* ModificationTracker)
{
	return RemoveDynamicLayersFromHierarchy({Filename}, ModificationTracker);
}

bool FConfigBranch::RemoveDynamicLayersFromHierarchy(const TArray<FString>& Filenames, FConfigModificationTracker* ModificationTracker)
{
	LLM_SCOPE(ELLMTag::ConfigSystem);

#if UE_THREAD_SAFETY_PROTECT_GETSTRING
	// this can modify InMemoryFile, so we need to protect it 
	UE::TUniqueLock WriteFileLock(InMemoryFile.SharedMutex);
#endif

	if (bIsSafeUnloaded)
	{
		SafeReload();
	}

	if (ReplayMethod == EBranchReplayMethod::NoReplay)
	{
		UE_LOGF(LogConfig, Warning, "Attempted to remove dynamic layer(s) from branch %ls, but it is using NoReplay mode, so this cannot work. Skipping.", *IniName.ToString());
		return false;
	}
	
	// calculate a patch so we don't lose in-memory changes
	FConfigCommandStream Patch = CalculateDiff(FinalCombinedLayers, InMemoryFile);

	for (const FString& Filename : Filenames)
	{
		for (DynamicLayerList::TIterator Node(DynamicLayers.GetHead()); Node; ++Node)
		{
			if (Node->Filename == Filename)
			{
				if (ModificationTracker != nullptr && ModificationTracker->bTrackModifiedSections)
				{
					for (const TPair<FString,FConfigCommandStreamSection>& Pair : **Node)
					{
						TSet<FString>& ModifiedSections = ModificationTracker->ModifiedSectionsPerBranch.FindOrAdd(IniName);
						ModifiedSections.Add(Pair.Key);
					}
				}

				// this will delete the layer
				DynamicLayers.RemoveNode(Node.GetNode());
				break;
			}
		}
	}
		
	// rebuild
	FinalCombinedLayers = CombinedStaticLayers;
	for (DynamicLayerList::TIterator Node(DynamicLayers.GetHead()); Node; ++Node)
	{
		FinalCombinedLayers.ApplyFile(*Node);
	}
	FinalCombinedLayers.ApplyFile(&CommandLineOverrides);
	bool bOldSaveAllSections = InMemoryFile.bCanSaveAllSections;
	InMemoryFile = FinalCombinedLayers;
	InMemoryFile.bCanSaveAllSections = bOldSaveAllSections;

	FinalCombinedLayers.Shrink();
	InMemoryFile.Shrink();

	// re-apply the in-memory changes
	InMemoryFile.ApplyFile(&Patch);
	
	return true;

}

void FConfigBranch::RemoveTagsFromHierarchy(const TArray<FName>& Tags, FConfigModificationTracker* ModificationTracker)
{
#if UE_THREAD_SAFETY_PROTECT_GETSTRING
	// this can modify InMemoryFile, so we need to protect it 
	UE::TUniqueLock WriteFileLock(InMemoryFile.SharedMutex);
#endif

	TArray<FString> LayersToRemove;

	
	// gather tagged layers
	for (DynamicLayerList::TIterator Node(DynamicLayers.GetHead()); Node; ++Node)
	{
		if (Tags.Contains(Node->Tag))
		{
			UE_LOGF(LogConfig, Verbose, "Removing dynamic layer %ls from branch %ls with tag %ls", *Node->Filename, *IniName.ToString(), *Node->Tag.ToString());
			// @todo make a version that takes CommandStreams, not Filenames, for speed
			LayersToRemove.Add(Node->Filename);
		}
	}
	
	// remove them
	if (LayersToRemove.Num() > 0)
	{
		RemoveDynamicLayersFromHierarchy(LayersToRemove, ModificationTracker);
	}
}

void FConfigBranch::SafeUnload()
{
#if UE_THREAD_SAFETY_PROTECT_GETSTRING
	// this can modify InMemoryFile, so we need to protect it 
	UE::TUniqueLock WriteLock(InMemoryFile.SharedMutex);
#endif
	
	bIsSafeUnloaded = true;
	InactiveTimer = 0;

#if UE_THREAD_SAFETY_USE_WRITE_LOCKS
	// if we are not using full Write Locks, then we we no longer unload the final combined data
	// this makes thread safety much simpler at the expense of some memory. because we don't clear 
	// the InMemoryFile, we don't actually need to SafeReload to get the final value, only when we 
	// add/remove layers or similar. 
	// This block is the low-memory, slower threading, mode
	// advanced behavior
	InMemoryFile.Cleanup();
#endif
	
	CombinedStaticLayers.Cleanup();
	FinalCombinedLayers.Cleanup();
	
	// empty the command streams for the static and dynamic layers, but leave any other streams alone
	// note that we keep the dynamic list around, but without the section data, because we use the 
	// dynamic layer filename, tag, and priority to load again
	StaticLayers.Empty();
	for (DynamicLayerList::TIterator Node(DynamicLayers.GetHead()); Node; ++Node)
	{
		if (!Node->bNeverSafeUnload)
		{
			Node->Empty();
		}
	}

	FCoreDelegates::TSOnConfigBranchUnloaded().Broadcast(*IniName.ToString());
}

void FConfigBranch::SafeReload()
{
	LLM_SCOPE(ELLMTag::ConfigSystem);

	double StartTime = FPlatformTime::Seconds();

	// read dynamic layers back in from disk
	TArray<FDynamicLayerInfo> ReloadInfos;
	
#if UE_THREAD_SAFETY_PROTECT_GETSTRING
	UE::TUniqueLock FileWriteLock(InMemoryFile.SharedMutex);
#endif

	// in case another thread reloaded while we were blocked, skip this
	if (!bIsSafeUnloaded)
	{
		return;
	}
	
	// read static layers back in from disk
	// @todo make sure we only Unload from GConfig
	FConfigContext Context = FConfigContext::ReadIntoConfigSystem(GConfig, Platform.ToString());
	Context.Branch = this;
	Context.DestIniFilename = IniPath;
	Context.bIsSafeReloading = true;
	Context.Load(*IniName.ToString());
			
	FConfigBranch::DynamicLayerList::TDoubleLinkedListNode* CurrentNode = DynamicLayers.GetHead(), * NextNode;
	while (CurrentNode != nullptr)
	{
		NextNode = CurrentNode->GetNextNode();
		
		// any never unload layers we just leave in the stream
		FConfigCommandStream& S = *CurrentNode->GetValue();
		if (!S.bNeverSafeUnload)
		{
			ReloadInfos.Add({ S.Filename, S.Tag, S.Priority });
			DynamicLayers.RemoveNode(CurrentNode, false);
		}
		CurrentNode = NextNode;
	}

	// even if we have no ReloadInfos, we may have leftover some string-based (bNeverSafeUnload) layers in the list, and we
	// will need to perform a full dynamic layer fixup of the final InMemoryFile in the branch - and we force a full fat 
	// update in either case to be 100% safe
	if (ReloadInfos.Num() > 0 || !DynamicLayers.IsEmpty())
	{
		const bool bForceFullDynamicLayerUpdate = true;
		AddDynamicLayersToHierarchy(ReloadInfos, nullptr, nullptr, nullptr, bForceFullDynamicLayerUpdate);
	}

	const FString IniNameStr = IniName.ToString();
	const float ReloadTimeMs = (float)(FPlatformTime::Seconds() - StartTime) * 1000.0f;
	UE_LOGF(LogConfig, Log, "Branch '%ls' had been unloaded. Reloading on-demand took %.2fms", *IniNameStr, ReloadTimeMs);

	const FConfigLogContext* LogContext = GetOutermostConfigLogContext();
	const TCHAR* ContextAction = LogContext->Action ? LogContext->Action : TEXT("");
	const TCHAR* ContextSection = LogContext->Section ? LogContext->Section : TEXT("");
	const TCHAR* ContextKey = LogContext->Key ? LogContext->Key : TEXT("");
	UE_LOGF(LogConfig, Log, "Reload Context for thread %ls: Action=%ls Section=%ls Key=%ls", *FThreadManager::GetThreadName(FPlatformTLS::GetCurrentThreadId()), ContextAction, ContextSection, ContextKey);

	FCoreDelegates::TSOnConfigBranchReloaded().Broadcast(*IniNameStr, ReloadTimeMs);
}

void FConfigBranch::UpdateInactiveTimer()
{
	if (InactiveTimer >= 0 && GTimeToUnloadConfig > 0)
	{
		InactiveTimer = FPlatformTime::Seconds();
		UE_LOGF(LogConfig, Verbose, "Resetting InactiveTimer for %ls", *IniName.ToString());
	}
}

void FConfigBranch::ReapplyLayers()
{
#if UE_THREAD_SAFETY_PROTECT_GETSTRING
	UE::TUniqueLock FileWriteLock(InMemoryFile.SharedMutex);
#endif
	
	// calculate a patch so we don't lose in-memory changes
	FConfigCommandStream Patch = CalculateDiff(FinalCombinedLayers, InMemoryFile);

	CombinedStaticLayers.Cleanup();
	FinalCombinedLayers.Cleanup();
	InMemoryFile.Cleanup();

	for (TPair<FString, FConfigCommandStream>& Pair : StaticLayers)
	{
		CombinedStaticLayers.ApplyFile(&Pair.Value);
	}

	FinalCombinedLayers = CombinedStaticLayers;
	for (DynamicLayerList::TIterator Node(DynamicLayers.GetHead()); Node; ++Node)
	{
		FinalCombinedLayers.ApplyFile(*Node);
	}

	FinalCombinedLayers.ApplyFile(&CommandLineOverrides);
	InMemoryFile = FinalCombinedLayers;
	InMemoryFile.ApplyFile(&Patch);
}

bool FConfigBranch::MergeStaticLayersUpTo(const FString& LayerSubstring, FConfigFile& OutFile) const
{
	for (const TPair<FString, FConfigCommandStream>& Pair : StaticLayers)
	{
		if (Pair.Key.Contains(LayerSubstring))
		{
			return true;
		}
		OutFile.ApplyFile(&Pair.Value);
	}

	return false;
}

bool FConfigBranch::MergeStaticLayersUpToAndIncluding(const FString& LayerSubstring, FConfigFile& OutFile) const
{
	for (const TPair<FString, FConfigCommandStream>& Pair : StaticLayers)
	{
		OutFile.ApplyFile(&Pair.Value);
		if (Pair.Key.Contains(LayerSubstring))
		{
			return true;
		}
	}

	return false;
}

const FConfigCommandStream* FConfigBranch::GetStaticLayer(const FString& LayerSubstring) const
{
	for (const TPair<FString, FConfigCommandStream>& Pair : StaticLayers)
	{
		if (Pair.Key.Contains(LayerSubstring))
		{
			return &Pair.Value;
		}
	}

	return nullptr;
}



bool FConfigBranch::RemoveSection(const TCHAR* Section)
{
	int NumRemoved = 0;
	
	FString SectionName(Section);
	for (TPair<FString, FConfigCommandStream>& Pair : StaticLayers)
	{
		NumRemoved += Pair.Value.Remove(SectionName);
	}
	for (DynamicLayerList::TIterator Node(DynamicLayers.GetHead()); Node; ++Node)
	{
		NumRemoved += Node->Remove(SectionName);
	}

	NumRemoved += InMemoryFile.Remove(SectionName);
	NumRemoved += CombinedStaticLayers.Remove(SectionName);
	NumRemoved += SavedLayer.Remove(SectionName);
	NumRemoved += CommandLineOverrides.Remove(SectionName);
	NumRemoved += FinalCombinedLayers.Remove(SectionName);

	return NumRemoved > 0;
}

bool FConfigBranch::Delete()
{
	bool bDeleted = DeleteConfigFileWrapper(*IniPath);
	return bDeleted;
}

void FConfigBranch::Shrink()
{
	RunOnEachFile([](FConfigFile& File, const FString& Name)
	{
		File.Shrink();
	});

	RunOnEachCommandStream([](FConfigCommandStream& Stream, const FString& Name)
	{
		Stream.Shrink();
	});
}

void FConfigBranch::Flush()
{
	SaveBranch(*this);
}

void FConfigBranch::Dump(FOutputDevice& Ar)
{
	Ar.Logf(TEXT("FConfigBranch %s"), *IniName.ToString());
	Ar.Logf(TEXT("Static Layers:"));
	for (const TPair<FString, FConfigCommandStream>& Pair : StaticLayers)
	{
		Ar.Logf(TEXT("  %s: %d sections"), *Pair.Key, Pair.Value.Num());
	}
	Ar.Logf(TEXT("Dynamic Layers:"));
	for (DynamicLayerList::TIterator Node(DynamicLayers.GetHead()); Node; ++Node)
	{
		Ar.Logf(TEXT("  %s: %d sections"), *Node->Filename, Node->Num());
	}
}

/*-----------------------------------------------------------------------------
	FConfigCacheIni
-----------------------------------------------------------------------------*/

namespace
{
	void OnConfigSectionsChanged(const FString& IniFilename, const TSet<FString>& SectionNames)
	{
		// when this is on, other code will do this in a way that doesn't force all ConsoleVariables cvars to be Hotfix level (see UE::DynamicConfig::PerformDynamicConfig)
		static bool bUseNewDynamicLayers = IConsoleManager::Get().FindConsoleVariable(TEXT("ini.UseNewDynamicLayers"))->GetInt() != 0;
		if (bUseNewDynamicLayers)
		{
			return;
		}

		if (IniFilename == GEngineIni && SectionNames.Contains(TEXT("ConsoleVariables")))
		{
			UE::ConfigUtilities::ApplyCVarSettingsFromIni(TEXT("ConsoleVariables"), *GEngineIni, ECVF_SetByHotfix);
		}
	}
}

#if WITH_EDITOR
static TMap<FName, TFuture<void>>& GetPlatformConfigFutures()
{
	static TMap<FName, TFuture<void>> Futures;
	return Futures;
}
#endif

FConfigCacheIni::FConfigCacheIni(EConfigCacheType InType, FName InPlatformName, bool bInGloballyRegistered)
	: bAreFileOperationsDisabled(false)
	, bIsReadyForUse(false)
	, bGloballyRegistered(bInGloballyRegistered)
	, Type(InType)
	, PlatformName(InPlatformName)
{
}

FConfigCacheIni::FConfigCacheIni()
{
	EnsureRetrievingVTablePtrDuringCtor(TEXT("FConfigCacheIni()"));
}

FConfigCacheIni::~FConfigCacheIni()
{
	// this destructor can run at file scope, static shutdown
	Flush( 1 );
}

bool FConfigCacheIni::IsConfigBranchNameInNeverUnloadList(const FName& ConfigBranchName)
{
	// No branch names to filter
	if (GConfigBranchesToNeverUnload.Len() == 0)
	{
		return false;
	}

	// Fill out the list first time encountered
	if (ConfigBranchNamesToNeverUnload.Num() == 0)
	{
		GConfigBranchesToNeverUnload.ParseIntoArray(ConfigBranchNamesToNeverUnload, TEXT(","));
	}

	if (ConfigBranchNamesToNeverUnload.Num() == 0)
	{
		return false;
	}

	const FString IniName = ConfigBranchName.ToString();
	return ConfigBranchNamesToNeverUnload.Contains(IniName);
}

void FConfigCacheIni::Tick(float DeltaSeconds)
{
	if (GTimeToUnloadConfig == 0)
	{
		return;
	}

	FConfigBranch* BranchesToCheck[2];

	// find next known file to check
	static int KnownFileToCheckForUnload = 0;
	if (KnownFileToCheckForUnload >= (uint8)EKnownIniFile::NumKnownFiles)
	{
		KnownFileToCheckForUnload = 0;
	}
	BranchesToCheck[0] = &KnownFiles.Branches[KnownFileToCheckForUnload++];
	
	// find next unknown file to check
	static int OtherFileToCheckForUnload = 0;
	if (OtherFileToCheckForUnload >= OtherFileNames.Num())
	{
		OtherFileToCheckForUnload = 0;
	}
	{
		UE::TScopeLock Lock(OtherFilesLock);
		BranchesToCheck[1] = OtherFiles.FindRef(OtherFileNames[OtherFileToCheckForUnload++]);
		checkf(OtherFileNames.Num() == OtherFiles.Num(), TEXT("OtherFiles and OtherFileNames are out of sync! %d other files, %d other file names!"), OtherFileNames.Num(), OtherFiles.Num());
	}
	
	// now check for unused files
	double Now = FPlatformTime::Seconds();
	for (FConfigBranch* Branch : BranchesToCheck)
	{
		if (Branch == nullptr || Branch->bIsSafeUnloaded || !Branch->bAllowedToRemove)
		{
			continue;
		}

		if (IsConfigBranchNameInNeverUnloadList(Branch->IniName))
		{
			Branch->bAllowedToRemove = false;
			continue;
		}
		
		// we start out negative so that we ignre the long startup time without ticking, so on first tick we allow it to be tracked
		if (Branch->InactiveTimer < 0)
		{
			Branch->InactiveTimer = Now;
		}
		else if (Branch->InactiveTimer > 0)
		{
			if (Now - Branch->InactiveTimer > GTimeToUnloadConfig)
			{
				UE_LOGF(LogConfig, Log, "Unloading %ls due to inactivity", *Branch->IniPath);
				
				Branch->SafeUnload();
			}
		}
	}
}



FConfigBranch* FConfigCacheIni::FindBranchWithNoReload(FName BaseIniName, const FString& Filename)
{
	// look for a known file, if there's no ini extension
	FConfigBranch* Branch = BaseIniName == NAME_None ? nullptr : KnownFiles.GetBranch(BaseIniName);

	if (Branch == nullptr)
	{
		Branch = KnownFiles.GetBranch(*Filename);
	}
	if (Branch == nullptr)
	{
		UE::TScopeLock Lock(OtherFilesLock);
		Branch = OtherFiles.FindRef(Filename);
		if (Branch == nullptr)
		{
			for (TPair<FString, FConfigBranch*>& CurrentFilePair : OtherFiles)
			{
				if (CurrentFilePair.Value->IniName == BaseIniName)
				{
					Branch = CurrentFilePair.Value;
					break;
				}
			}
		}
	}

	// if Filename is a .ini and it doesn't match what the KnownFile has (if it has one yet), then we can't use it
	if (Branch && Branch->IniPath.Len() > 0 && Filename.Len() > 0 && Filename.EndsWith(".ini") && Branch->IniPath != Filename)
	{
		Branch = nullptr;
	}

	if (Branch)
	{
		// track that this branch is being used, so re-set the time
		Branch->UpdateInactiveTimer();
	}

	return Branch;
}

FConfigBranch* FConfigCacheIni::FindBranchWithDynamicLayerTag(FName BaseIniName, FName Tag)
{
	FConfigBranch* Branch = FindBranch(BaseIniName, "");

	if (Branch)
	{
		for (FConfigBranch::DynamicLayerList::TIterator Node(Branch->DynamicLayers.GetHead()); Node; ++Node)
		{
			if (Node->Tag == Tag)
			{
				return Branch;
			}
		}
	}

	return nullptr;
}

FConfigBranch* FConfigCacheIni::FindBranch(FName BaseIniName, const FString& Filename)
{
	FConfigLogContextScope LogScope(TEXT("FindBranch"), nullptr, nullptr);

	FConfigBranch* Branch = FindBranchWithNoReload(BaseIniName, Filename);

	if (Branch)
	{
		if (Branch->bIsSafeUnloaded)
		{
			Branch->SafeReload();
		}

		// track that this branch is being used, so re-set the time
		Branch->UpdateInactiveTimer();
	}

	return Branch;
}

FConfigBranch& FConfigCacheIni::AddNewBranch(const FString& Filename)
{
	FConfigBranch* Branch = new FConfigBranch();
	Branch->IniName = *FPaths::GetBaseFilename(Filename);
	Branch->IniPath = Filename;
#if UE_WITH_CONFIG_TRACKING
	UE::ConfigAccessTracking::FFile* FileAccess = Branch->InMemoryFile.GetFileAccess();
	if (FileAccess)
	{
		FileAccess->SetAsLoadTypeConfigSystem(*this, Branch->InMemoryFile);
		FileAccess->OverrideFilenameToLoad = FName(FStringView(Filename));
	}
#endif
	
	UE::TScopeLock Lock(OtherFilesLock);
	if (OtherFiles.Find(Filename) == nullptr)
	{
		OtherFileNames.Add(Filename);
	}
	FConfigBranch*& Existing = OtherFiles.FindOrAdd(Filename);
	delete Existing;
	Existing = Branch;
	return *Branch;
}

int32 FConfigCacheIni::Remove(const FString& Filename)
{
	UE::TScopeLock Lock(OtherFilesLock);

	OtherFileNames.Remove(Filename);
	delete OtherFiles.FindRef(Filename);
	return OtherFiles.Remove(Filename);
}


FConfigFile* FConfigCacheIni::FindConfigFile( const FString& Filename )
{
	FConfigLogContextScope LogScope(TEXT("FindConfigFile"), nullptr, nullptr);

	FConfigBranch* Result;
	if (!Filename.EndsWith(TEXT(".ini")))
	{
		Result = KnownFiles.GetBranch(*Filename);
	}
	else
	{
		UE::TScopeLock Lock(OtherFilesLock);
		Result = OtherFiles.FindRef(Filename);
	}

	if (Result)
	{
		if (Result->bIsSafeUnloaded)
		{
			Result->SafeReload();
		}

		// track that this branch is being used, so re-set the time
		Result->UpdateInactiveTimer();

		return &Result->InMemoryFile;
	}

	return nullptr;
}

FConfigFile* FConfigCacheIni::Find(const FString& Filename)
{
	FConfigLogContextScope LogScope(TEXT("Find"), nullptr, nullptr);

	// check for non-filenames
	if(Filename.Len() == 0)
	{
		return nullptr;
	}

	// Get the file if it exists
	FConfigFile* Result = FindConfigFile(Filename);

	// this is || filesize so we load up .int files if file IO is allowed
	if (!Result && !bAreFileOperationsDisabled)
	{
		// Before attempting to add another file, double check that this doesn't exist at a normalized path.
		const FString UnrealFileName = NormalizeConfigIniPath(Filename);
		Result = FindConfigFile(UnrealFileName);
		
		if (!Result)
		{
			if (DoesConfigFileExistWrapper(*UnrealFileName))
			{
				Result = &Add(UnrealFileName, FConfigFile());
				UE_LOGF(LogConfig, Verbose, "GConfig::Find is looking for file:  %ls", *UnrealFileName);
				{
					// Files added through Find are treated the same as ReadSingleIntoConfigSystem contexts,
					// and do not use a hierarchy so they do not use a generatedini and should never be saved.
					Result->NoSave = true;
#if UE_WITH_CONFIG_TRACKING
					Result->LoadType = UE::ConfigAccessTracking::ELoadType::LocalSingleIniFile;
					UE::ConfigAccessTracking::FFile* FileAccess = Result->GetFileAccess();
					if (FileAccess)
					{
						FileAccess->OverrideFilenameToLoad = FName(FStringView(UnrealFileName));
					}
#endif
					Result->Read(UnrealFileName);
					UE_LOGF(LogConfig, Verbose, "GConfig::Find has loaded file:  %ls", *UnrealFileName);
				}
			}
		}
		else
		{
			// We could normalize always normalize paths, but we don't want to always incur the penalty of that
			// when callers can cache the strings ahead of time.
			UE_LOGF(LogConfig, Warning, "GConfig::Find attempting to access config with non-normalized path %ls. Please use FConfigCacheIni::NormalizeConfigIniPath (which would make generate %ls) before accessing INI files through ConfigCache.", *Filename, *UnrealFileName);
		}
	}

	return Result;
}

FConfigFile* FConfigCacheIni::FindConfigFileWithBaseName(FName BaseName)
{
	FConfigLogContextScope LogScope(TEXT("FindConfigFileWithBaseName"), nullptr, nullptr);

	FConfigBranch* Result = KnownFiles.GetBranch(BaseName);
	if (Result == nullptr)
	{
		UE::TScopeLock Lock(OtherFilesLock);
		for (TPair<FString,FConfigBranch*>& CurrentFilePair : OtherFiles)
		{
			if (CurrentFilePair.Value->IniName == BaseName)
			{
				Result = CurrentFilePair.Value;
				break;
			}
		}
	}

	if (Result)
	{
		if (Result->bIsSafeUnloaded)
	{
			Result->SafeReload();
		}

		// track that this branch is being used, so re-set the time
		Result->UpdateInactiveTimer();

		return &Result->InMemoryFile;
	}
	return nullptr;
}

FConfigFile& FConfigCacheIni::Add(const FString& Filename, const FConfigFile& File)
{
	FConfigBranch* Branch = new FConfigBranch(File);
	Branch->IniName = File.Name;
	Branch->IniPath = Filename;
#if UE_WITH_CONFIG_TRACKING
	UE::ConfigAccessTracking::FFile* FileAccess = Branch->InMemoryFile.GetFileAccess();
	if (FileAccess)
	{
		FileAccess->SetAsLoadTypeConfigSystem(*this, Branch->InMemoryFile);
		FileAccess->OverrideFilenameToLoad = FName(FStringView(Filename));
	}
#endif

	UE::TScopeLock Lock(OtherFilesLock);

	if (OtherFiles.Find(Filename) == nullptr)
	{
		OtherFileNames.Add(Filename);
	}
	FConfigBranch*& Existing = OtherFiles.FindOrAdd(Filename);
	delete Existing;
	Existing = Branch;
	return Branch->InMemoryFile;
}

bool FConfigCacheIni::ContainsConfigFile(const FConfigFile* ConfigFile) const
{
	// Check the normal inis. Note that the FConfigFiles in the map
	// could have been reallocated if new inis were added to the map
	// since the point at which the caller received the ConfigFile pointer
	// they are testing. It is the caller's responsibility to not try to hold
	// on to the ConfigFile pointer during writes to this ConfigCacheIni
	{
		UE::TScopeLock Lock(OtherFilesLock);
		for (const TPair<FString, FConfigBranch*>& CurrentFilePair : OtherFiles)
		{
			if (ConfigFile == &CurrentFilePair.Value->InMemoryFile)
			{
				return true;
			}
		}
	}
	// Check the known inis
	for (const FConfigBranch& Branch : KnownFiles.Branches)
	{
		if (ConfigFile == &Branch.InMemoryFile)
		{
			return true;
		}
	}

	return false;
}


TArray<FString> FConfigCacheIni::GetFilenames()
{	
	TArray<FString> Result;
	{
		UE::TScopeLock Lock(OtherFilesLock);
		Result = OtherFileNames;
	}

	for (const FConfigBranch& Branch : KnownFiles.Branches)
	{
		Result.Add(Branch.IniName.ToString());
	}

	return Result;
}

bool FConfigCacheIni::Flush(bool bRemoveFromCache, const FString& Filename)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FConfigCacheIni::Flush);

	bool bSuccess = true;

	// never Flush temporary cache objects
	if (Type != EConfigCacheType::Temporary)
	{
		// write out the files if we can
		if (!bAreFileOperationsDisabled)
		{
			if (Filename.Len() > 0)
			{
				// flush single file
				if (FConfigBranch* Branch = FindBranch(*Filename, Filename))
				{
					bSuccess = SaveBranch(*Branch) && bSuccess;
				}
			}
			else
			{
				UE::TScopeLock Lock(OtherFilesLock);
				// flush all files
				for (TPair<FString, FConfigBranch*>& Pair : OtherFiles)
				{
					bSuccess = SaveBranch(*Pair.Value) && bSuccess;
				}
				for (FConfigBranch& Branch : KnownFiles.Branches)
				{
					bSuccess = SaveBranch(Branch) && bSuccess;
				}
			}
		}
	}

	if (bRemoveFromCache)
	{
		// we can't read it back in if file operations are disabled
		if (bAreFileOperationsDisabled)
		{
			UE_LOGF(LogConfig, Warning, "Tried to flush the config cache and read it back in, but File Operations are disabled!!");
		}
		else if (Filename.Len() != 0)
		{
			Remove(Filename);
		}
		else
		{
			UE::TScopeLock Lock(OtherFilesLock);
			for (TPair<FString, FConfigBranch*>& It : OtherFiles)
			{
				delete It.Value;
			}
			OtherFiles.Empty();
			OtherFileNames.Empty();
		}
	}

	return bSuccess;
}

/**
 * Disables any file IO by the config cache system
 */
void FConfigCacheIni::DisableFileOperations()
{
	bAreFileOperationsDisabled = true;
}

/**
 * Re-enables file IO by the config cache system
 */
void FConfigCacheIni::EnableFileOperations()
{
	bAreFileOperationsDisabled = false;
}

/**
 * Returns whether or not file operations are disabled
 */
bool FConfigCacheIni::AreFileOperationsDisabled()
{
	return bAreFileOperationsDisabled;
}

/**
 * Parses apart an ini section that contains a list of 1-to-N mappings of names in the following format
 *	 [PerMapPackages]
 *	 .MapName1=Map1
 *	 .Package1=PackageA
 *	 .Package1=PackageB
 *	 .MapName2=Map2
 *	 .Package2=PackageC
 *	 .Package2=PackageD
 * 
 * @param Section Name of section to look in
 * @param KeyOne Key to use for the 1 in the 1-to-N (MapName in the above example - the number suffix gets ignored but helps to keep ordering)
 * @param KeyN Key to use for the N in the 1-to-N (Package in the above example - the number suffix gets ignored but helps to keep ordering)
 * @param OutMap Map containing parsed results
 * @param Filename Filename to use to find the section
 *
 * NOTE: The function naming is weird because you can't apparently have an overridden function differnt only by template type params
 */
void FConfigCacheIni::Parse1ToNSectionOfNames(const TCHAR* Section, const TCHAR* KeyOne, const TCHAR* KeyN, TMap<FName, TArray<FName> >& OutMap, const FString& Filename)
{
	FConfigLogContextScope LogScope(TEXT("Parse1ToNSectionOfNames"), Section, KeyOne);

	// find the config file object
	FConfigFile* ConfigFile = Find(Filename);
	if (!ConfigFile)
	{
		return;
	}

	// find the section in the file
	const FConfigSectionMap* ConfigSection = ConfigFile->FindSection(Section);
	if (!ConfigSection)
	{
		return;
	}

	TArray<FName>* WorkingList = nullptr;
	for( FConfigSectionMap::TConstIterator It(*ConfigSection); It; ++It )
	{
		// is the current key the 1 key?
		if (It.Key().ToString().StartsWith(KeyOne))
		{
			const FName KeyName(*It.Value().GetValue());

			// look for existing set in the map
			WorkingList = OutMap.Find(KeyName);

			// make a new one if it wasn't there
			if (WorkingList == nullptr)
			{
				WorkingList = &OutMap.Add(KeyName, TArray<FName>());
			}
		}
		// is the current key the N key?
		else if (It.Key().ToString().StartsWith(KeyN) && WorkingList != nullptr)
		{
			// if so, add it to the N list for the current 1 key
			WorkingList->Add(FName(*It.Value().GetValue()));
		}
		// if it's neither, then reset
		else
		{
			WorkingList = nullptr;
		}
	}
}

/**
 * Parses apart an ini section that contains a list of 1-to-N mappings of strings in the following format
 *	 [PerMapPackages]
 *	 .MapName1=Map1
 *	 .Package1=PackageA
 *	 .Package1=PackageB
 *	 .MapName2=Map2
 *	 .Package2=PackageC
 *	 .Package2=PackageD
 * 
 * @param Section Name of section to look in
 * @param KeyOne Key to use for the 1 in the 1-to-N (MapName in the above example - the number suffix gets ignored but helps to keep ordering)
 * @param KeyN Key to use for the N in the 1-to-N (Package in the above example - the number suffix gets ignored but helps to keep ordering)
 * @param OutMap Map containing parsed results
 * @param Filename Filename to use to find the section
 *
 * NOTE: The function naming is weird because you can't apparently have an overridden function differnt only by template type params
 */
void FConfigCacheIni::Parse1ToNSectionOfStrings(const TCHAR* Section, const TCHAR* KeyOne, const TCHAR* KeyN, TMap<FString, TArray<FString> >& OutMap, const FString& Filename)
{
	FConfigLogContextScope LogScope(TEXT("Parse1ToNSectionOfStrings"), Section, KeyOne);

	// find the config file object
	FConfigFile* ConfigFile = Find(Filename);
	if (!ConfigFile)
	{
		return;
	}

	// find the section in the file
	const FConfigSectionMap* ConfigSection = ConfigFile->FindSection(Section);
	if (!ConfigSection)
	{
		return;
	}

	TArray<FString>* WorkingList = nullptr;
	for( FConfigSectionMap::TConstIterator It(*ConfigSection); It; ++It )
	{
		// is the current key the 1 key?
		if (It.Key().ToString().StartsWith(KeyOne))
		{
			// look for existing set in the map
			WorkingList = OutMap.Find(It.Value().GetValue());

			// make a new one if it wasn't there
			if (WorkingList == nullptr)
			{
				WorkingList = &OutMap.Add(It.Value().GetValue(), TArray<FString>());
			}
		}
		// is the current key the N key?
		else if (It.Key().ToString().StartsWith(KeyN) && WorkingList != nullptr)
		{
			// if so, add it to the N list for the current 1 key
			WorkingList->Add(It.Value().GetValue());
		}
		// if it's neither, then reset
		else
		{
			WorkingList = nullptr;
		}
	}
}

void FConfigCacheIni::LoadFile( const FString& Filename, const FConfigFile* Fallback, const TCHAR* PlatformString )
{
	// if the file has some data in it, read it in
	if( !IsUsingLocalIniFile(*Filename, nullptr) || DoesConfigFileExistWrapper(*Filename) )
	{
		FConfigFile* Result = &Add(Filename, FConfigFile());
		bool bDoEmptyConfig = false;
		bool bDoCombine = false;
		ProcessIniContents(*Filename, *Filename, Result, bDoEmptyConfig, bDoCombine);
		UE_LOGF(LogConfig, Verbose, "GConfig::LoadFile has loaded file:  %ls", *Filename);
	}
	else if( Fallback )
	{
		Add( *Filename, *Fallback );
		UE_LOGF(LogConfig, Verbose, "GConfig::LoadFile associated file:  %ls", *Filename);
	}
	else
	{
		UE_LOGF(LogConfig, Warning, "FConfigCacheIni::LoadFile failed loading file as it was 0 size.  Filename was:  %ls", *Filename);
	}
}


void FConfigCacheIni::SetFile( const FString& Filename, const FConfigFile* NewConfigFile )
{
	if (FConfigFile* FoundFile = KnownFiles.GetMutableFile(FName(*Filename, FNAME_Find)))
	{
		*FoundFile = *NewConfigFile;
	}
	else
	{
		Add(Filename, *NewConfigFile);
	}
}


void FConfigCacheIni::UnloadFile(const FString& Filename)
{
	FConfigFile* File = Find(Filename);
	if( File )
		Remove( Filename );
}

void FConfigCacheIni::Detach(const FString& Filename)
{
	FConfigFile* File = Find(Filename);
	if( File )
		File->NoSave = 1;
}

static FConfigBranch* FindOrLoadNoSafeReload(FConfigCacheIni* ConfigSystem, const FString& Filename)
{
	// check for non-filenames
	if(Filename.Len() == 0)
	{
		return nullptr;
	}

	// we need to get the branch without reloading here so that we can put the Reload and Get() be protected
	// by the same block, which is held buy the the Branch/File we are looking up
	FConfigBranch* Branch = ConfigSystem->FindBranchWithNoReload(NAME_None, Filename);
	if (Branch == nullptr)
	{
		// this will load if possible, with Normalized path (and log a warning if it wasn't match)
		ConfigSystem->Find(*Filename);
		
		// need to find with Normalized path
		Branch = ConfigSystem->FindBranchWithNoReload(NAME_None, ConfigSystem->NormalizeConfigIniPath(Filename));
	}

	return Branch;
}

bool FConfigCacheIni::GetString( const TCHAR* Section, const TCHAR* Key, FString& Value, const FString& Filename )
{
	FConfigLogContextScope LogScope(TEXT("GetString"), Section, Key);

	// get the file without a SafeReload possible, so no lock is needed
	FConfigBranch* Branch = FindOrLoadNoSafeReload(this, Filename);
	if (Branch == nullptr)
	{
		return false;
	}
	
	// due to some rare-but-possible edge cases, we need to either keep InMemoryFile around when we 
	// SafeUnload, and _not_ call SafeReload here, or we need a writelock around both the Unload and the
	// GetString functionality (or we would Unload between the Reload call and the Get). A ReadLock isn't
	// enough, and was causing deadlocks
#if UE_THREAD_SAFETY_USE_WRITE_LOCKS
	// don't read while something may be modifying the File (generally a Branch's InMmeoryFile)
	// frustratingly we need 
	UE::TUniqueLock WriteLock(Branch->InMemoryFile.SharedMutex);
	
	if (Branch->bIsSafeUnloaded)
	{
		Branch->SafeReload();
	}
#else
	// we do need a ReadLock because dynamiclayers can still modify InMemoryFile
	// (if we did a SafeReload, we would need a WriteLock which makes this function ~2-4x slower!)
	UE::TSharedLock ReadLock(Branch->InMemoryFile.SharedMutex);	
#endif
	
	FRemoteConfig::Get()->FinishRead(*Filename); // Ensure the remote file has been loaded and processed

	
	const FConfigSection* Sec = Branch->InMemoryFile.FindSection( Section );
	if( !Sec )
	{
#if !UE_BUILD_SHIPPING
		CheckLongSectionNames( Section, &Branch->InMemoryFile );
#endif
		return false;
	}
	const FConfigValue* ConfigValue = Sec->Find( Key );
	if( !ConfigValue )
	{
		return false;
	}
	Value = ConfigValue->GetValue();

	FCoreDelegates::TSOnConfigValueRead().Broadcast(*Filename, Section, Key);

	return true;
}

bool FConfigCacheIni::GetText( const TCHAR* Section, const TCHAR* Key, FText& Value, const FString& Filename )
{
	FConfigLogContextScope LogScope(TEXT("GetText"), Section, Key);

	// get the file without a SafeReload possible, so no lock is needed
	FConfigBranch* Branch = FindOrLoadNoSafeReload(this, Filename);	
	if (Branch == nullptr)
	{
		return false;
	}

	// due to some rare-but-possible edge cases, we need to either keep InMemoryFile around when we 
	// SafeUnload, and _not_ call SafeReload here, or we need a writelock around both the Unload and the
	// GetString functionality (or we would Unload between the Reload call and the Get). A ReadLock isn't
	// enough, and was causing deadlocks
#if UE_THREAD_SAFETY_USE_WRITE_LOCKS
	// don't read while something may be modifying the File (generally a Branch's InMmeoryFile)
	// frustratingly we need 
	UE::TUniqueLock WriteLock(Branch->InMemoryFile.SharedMutex);
	
	if (Branch->bIsSafeUnloaded)
	{
		Branch->SafeReload();
	}
#else
	// we do need a ReadLock because dynamiclayers can still modify InMemoryFile
	// (if we did a SafeReload, we would need a WriteLock which makes this function ~2-4x slower!)
	UE::TSharedLock ReadLock(Branch->InMemoryFile.SharedMutex);	
#endif

	FRemoteConfig::Get()->FinishRead(*Filename); // Ensure the remote file has been loaded and processed
	
	const FConfigSection* Sec = Branch->InMemoryFile.FindSection( Section );
	if( !Sec )
	{
#if !UE_BUILD_SHIPPING
		CheckLongSectionNames( Section, &Branch->InMemoryFile );
#endif
		return false;
	}
	const FConfigValue* ConfigValue = Sec->Find( Key );
	if( !ConfigValue )
	{
		return false;
	}
	if (FTextStringHelper::ReadFromBuffer(*ConfigValue->GetValue(), Value, Section) == nullptr)
	{
		return false;
	}

	FCoreDelegates::TSOnConfigValueRead().Broadcast(*Filename, Section, Key);

	return true;
}

bool FConfigCacheIni::GetSection( const TCHAR* Section, TArray<FString>& Result, const FString& Filename )
{
	FConfigLogContextScope LogScope(TEXT("GetSection"), Section, nullptr);

	FRemoteConfig::Get()->FinishRead(*Filename); // Ensure the remote file has been loaded and processed
	Result.Reset();
	FConfigFile* File = Find(Filename);
	if (!File)
	{
		return false;
	}
	const FConfigSection* Sec = File->FindSection( Section );
	if (!Sec)
	{
		return false;
	}
	Result.Reserve(Sec->Num());
	for (FConfigSection::TConstIterator It(*Sec); It; ++It)
	{
		Result.Add(FString::Printf(TEXT("%s=%s"), *It.Key().ToString(), *It.Value().GetValue()));
	}

	FCoreDelegates::TSOnConfigSectionRead().Broadcast(*Filename, Section);

	return true;
}

const FConfigSection* FConfigCacheIni::GetSection( const TCHAR* Section, const bool Force, const FString& Filename )
{
	FConfigLogContextScope LogScope(TEXT("GetSection"), Section, nullptr);

	FRemoteConfig::Get()->FinishRead(*Filename); // Ensure the remote file has been loaded and processed
	FConfigFile* File = Find(Filename);
	if (!File)
	{
		return nullptr;
	}
	const FConfigSection* Sec = File->FindSection( Section );
	if (!Sec && Force)
	{
		UE::ConfigAccessTracking::FSection* SectionAccess = nullptr;
#if UE_WITH_CONFIG_TRACKING
		UE::ConfigAccessTracking::FFile* LocalFileAccess = File->GetFileAccess();
		SectionAccess = LocalFileAccess ? new UE::ConfigAccessTracking::FSection(*LocalFileAccess, FStringView(Section)) : nullptr;
#endif
		Sec = &File->Add(Section, FConfigSection(SectionAccess));
		File->Dirty = true;
	}

	if (Sec)
	{
		FCoreDelegates::TSOnConfigSectionRead().Broadcast(*Filename, Section);
	}

	return Sec;
}

bool FConfigCacheIni::DoesSectionExist(const TCHAR* Section, const FString& Filename)
{
	FConfigLogContextScope LogScope(TEXT("DoesSectionExist"), Section, nullptr);

	bool bReturnVal = false;

	FRemoteConfig::Get()->FinishRead(*Filename); // Ensure the remote file has been loaded and processed
	FConfigFile* File = Find(Filename);

	bReturnVal = (File != nullptr && File->FindSection(Section) != nullptr);

	if (bReturnVal)
	{
		FCoreDelegates::TSOnConfigSectionNameRead().Broadcast(*Filename, Section);
	}

	return bReturnVal;
}

void FConfigCacheIni::SetString( const TCHAR* Section, const TCHAR* Key, const TCHAR* Value, const FString& Filename )
{
	FConfigFile* File = Find(Filename);

	if (!File)
	{
		return;
	}

	File->SetString(Section, Key, Value);
}

void FConfigCacheIni::SetText( const TCHAR* Section, const TCHAR* Key, const FText& Value, const FString& Filename )
{
	FConfigFile* File = Find(Filename);

	if ( !File )
	{
		return;
	}

	FConfigSection* Sec = File->FindOrAddSectionInternal( Section );

	FString StrValue;
	FTextStringHelper::WriteToBuffer(StrValue, Value);

	FConfigValue* ConfigValue = Sec->Find( Key );
	if( !ConfigValue )
	{
		Sec->Add(Key, FConfigValue(Sec, FName(Key), MoveTemp(StrValue)));
		File->Dirty = true;
	}
	// Use GetSavedValueForWriting rather than GetSavedValue to avoid reporting the is-it-dirty query mark the values as having been accessed for dependency tracking
	else if( FCString::Strcmp(*ConfigValue->GetSavedValueForWriting(), *StrValue)!=0 )
	{
		File->Dirty = true;
		*ConfigValue = MoveTemp(StrValue);
	}
}

bool FConfigCacheIni::RemoveKey( const TCHAR* Section, const TCHAR* Key, const FString& Filename )
{
	FConfigFile* File = Find(Filename);
	if( File )
	{
		if (File->RemoveKeyFromSection(Section, Key))
			{
				File->Dirty = 1;
				return true;
			}
		}
	return false;
}

bool FConfigCacheIni::SafeUnloadBranch(const TCHAR* BranchName)
{
	FConfigBranch* Branch = FindBranchWithNoReload(BranchName, BranchName);
	if (Branch)
	{
		Branch->SafeUnload();
		return true;
	}

	return false;
}

bool FConfigCacheIni::RemoveSectionFromBranch(const TCHAR* Section, const TCHAR* Filename)
{
	FConfigBranch* Branch = FindBranchWithNoReload(Filename, Filename);
	if (Branch)
	{
		return Branch->RemoveSection(Section);
	}

	return false;	
}

bool FConfigCacheIni::EmptySection( const TCHAR* Section, const FString& Filename )
{
	FConfigFile* File = Find(Filename);
	if( File )
	{
		// remove the section name if there are no more properties for this section
		if(File->FindSection(Section) != nullptr)
		{
			File->Remove(Section);
			if (bAreFileOperationsDisabled == false)
			{
				if (File->Num())
				{
					File->Dirty = 1;
					Flush(0, Filename);
				}
				else
				{
					DeleteConfigFileWrapper(*Filename);
				}
			}
			return true;
		}
	}
	return false;
}

bool FConfigCacheIni::EmptySectionsMatchingString( const TCHAR* SectionString, const FString& Filename )
{
	bool bEmptied = false;
	FConfigFile* File = Find(Filename);
	if (File)
	{
		bool bSaveOpsDisabled = bAreFileOperationsDisabled;
		bAreFileOperationsDisabled = true;

		TArray<FString> SectionsToEmpty;
		for (FConfigFile::TConstIterator It = File->CreateConstIterator(); It; ++It)
		{
			if (It.Key().Contains(SectionString))
			{
				SectionsToEmpty.Add(It.Key());
			}
		}
				
		for (const FString& SectionNameString : SectionsToEmpty)
		{
			bEmptied |= EmptySection(*SectionNameString, Filename);
		}

		bAreFileOperationsDisabled = bSaveOpsDisabled;
	}
	return bEmptied;
}

FString FConfigCacheIni::GetConfigFilename(const TCHAR* BaseIniName)
{
	// Known ini files such as Engine, Game, etc.. are referred to as just the name with no extension within the config system.
	if (IsKnownConfigName(FName(BaseIniName, FNAME_Find)))
	{
		return FString(BaseIniName);
	}
	else
	{
		// Non-known ini files are looked up using their full path
		// This always uses the default platform as non-known files are not valid for other platforms
		return GetDestIniFilename(BaseIniName, nullptr, *FPaths::GeneratedConfigDir());
	}
}

/**
 * Retrieve a list of all of the config files stored in the cache
 *
 * @param ConfigFilenames Out array to receive the list of filenames
 */
void FConfigCacheIni::GetConfigFilenames(TArray<FString>& ConfigFilenames)
{
	ConfigFilenames = GetFilenames();
}

/**
 * Retrieve the names for all sections contained in the file specified by Filename
 *
 * @param	Filename			the file to retrieve section names from
 * @param	out_SectionNames	will receive the list of section names
 *
 * @return	true if the file specified was successfully found;
 */
bool FConfigCacheIni::GetSectionNames( const FString& Filename, TArray<FString>& out_SectionNames )
{
	FConfigLogContextScope LogScope(TEXT("GetSectionNames"), nullptr, nullptr);

	bool bResult = false;

	FConfigFile* File = Find(Filename);
	if ( File != nullptr )
	{
		out_SectionNames.Empty(File->Num());
		for ( FConfigFile::TIterator It(*File); It; ++It )
		{
			out_SectionNames.Add(It.Key());

			FCoreDelegates::TSOnConfigSectionNameRead().Broadcast(*Filename, *It.Key());
		}
		bResult = true;
	}

	return bResult;
}

/**
 * Retrieve the names of sections which contain data for the specified PerObjectConfig class.
 *
 * @param	Filename			the file to retrieve section names from
 * @param	SearchClass			the name of the PerObjectConfig class to retrieve sections for.
 * @param	out_SectionNames	will receive the list of section names that correspond to PerObjectConfig sections of the specified class
 * @param	MaxResults			the maximum number of section names to retrieve
 *
 * @return	true if the file specified was found and it contained at least 1 section for the specified class
 */
bool FConfigCacheIni::GetPerObjectConfigSections( const FString& Filename, const FString& SearchClass, TArray<FString>& out_SectionNames, int32 MaxResults )
{
	FConfigLogContextScope LogScope(TEXT("GetPerObjectConfigSections"), *SearchClass, nullptr);

	bool bResult = false;

	MaxResults = FMath::Max(0, MaxResults);
	FConfigFile* File = Find(Filename);
	if ( File != nullptr )
	{
		out_SectionNames.Empty();
		for ( FConfigFile::TIterator It(*File); It && out_SectionNames.Num() < MaxResults; ++It )
		{
			const FString& SectionName = It.Key();
			
			// determine whether this section corresponds to a PerObjectConfig section
			int32 POCClassDelimiter = SectionName.Find(TEXT(" "));
			if ( POCClassDelimiter != INDEX_NONE )
			{
				// the section name contained a space, which for now we'll assume means that we've found a PerObjectConfig section
				// see if the remainder of the section name matches the class name we're searching for
				if ( SectionName.Mid(POCClassDelimiter + 1) == SearchClass )
				{
					// found a PerObjectConfig section for the class specified - add it to the list
					out_SectionNames.Insert(SectionName,0);
					bResult = true;

					FCoreDelegates::TSOnConfigSectionNameRead().Broadcast(*Filename, *SectionName);
				}
			}
		}
	}

	return bResult;
}

void FConfigCacheIni::Exit()
{
	Flush( 1 );

#if WITH_EDITOR
	for (auto& PlatformConfigFuture : GetPlatformConfigFutures())
	{
		PlatformConfigFuture.Value.Get();
	}
	GetPlatformConfigFutures().Empty();
#endif
}

static void DumpBranch(FOutputDevice& Ar, const FConfigBranch& Branch)
{
	Ar.Logf(TEXT("Branch Name: %s"), *Branch.IniName.ToString());
	Ar.Logf(TEXT("Branch Filename: %s"), *Branch.IniPath);
	Ar.Logf(TEXT("Branch Static Hierarchy:"));
	for (const TPair<int32, FUtf8String>& Pair: Branch.Hierarchy)
	{
		Ar.Logf(TEXT("  %s"), *FString(Pair.Value));
	}
	if (Branch.DynamicLayers.GetHead())
	{
		Ar.Logf(TEXT("Branch Dynamic Layers:"));
		for (FConfigBranch::DynamicLayerList::TIterator Node(Branch.DynamicLayers.GetHead()); Node; ++Node)
		{
			Ar.Logf(TEXT("  %s"), *Node.GetNode()->GetValue()->Filename);
		}
	}

	Ar.Logf(TEXT("Branch Values:"));
	// sort the sections (and keys below) for easier diffing
	TArray<FString> SectionKeys;
	Branch.InMemoryFile.GetKeys(SectionKeys);
	SectionKeys.Sort();
	for (const FString& SectionKey : SectionKeys)
	{
		const FConfigSection* Sec = Branch.InMemoryFile.FindSection(SectionKey);
		Ar.Logf(TEXT("   [%s]"), *SectionKey);

		TArray<FName> Keys;
		Sec->GetKeys(Keys);
		Keys.Sort(FNameLexicalLess());
		for (FName Key : Keys)
		{
			TArray<FConfigValue> Values;
			Sec->MultiFind(Key, Values, true);
			for (const FConfigValue& Value : Values)
			{
				Ar.Logf(TEXT("   %s=%s"), *Key.ToString(), *Value.GetSavedValueForWriting());
			}
		}

		Ar.Log(LINE_TERMINATOR);
	}
}

void FConfigCacheIni::Dump(FOutputDevice& Ar, const TCHAR* BaseIniName)
{
	for (const FConfigBranch& Branch : KnownFiles.Branches)
	{
		if (BaseIniName == nullptr || Branch.IniName == BaseIniName)
		{
			DumpBranch(Ar, Branch);
		}
	}

	UE::TScopeLock Lock(OtherFilesLock);

	// sort the non-known files for easier diffing
	TArray<FString> Keys;
	OtherFiles.GetKeys(Keys);
	Algo::Sort(Keys);
	for (const FString& Key : Keys)
	{
		if (BaseIniName == nullptr || FPaths::GetBaseFilename(Key) == BaseIniName)
		{
			DumpBranch(Ar, OtherFiles[Key]->InMemoryFile);
		}
	}
}

// Derived functions.
FString FConfigCacheIni::GetStr( const TCHAR* Section, const TCHAR* Key, const FString& Filename )
{
	FString Result;
	GetString( Section, Key, Result, Filename );
	return Result;
}
bool FConfigCacheIni::GetInt
(
	const TCHAR*		Section,
	const TCHAR*		Key,
	int32&				Value,
	const FString&	Filename
)
{
	FString Text; 
	if( GetString( Section, Key, Text, Filename ) )
	{
		Value = FCString::Atoi(*Text);
		return true;
	}
	return false;
}
bool FConfigCacheIni::GetInt64
(
	const TCHAR* Section,
	const TCHAR* Key,
	int64& Value,
	const FString& Filename
)
{
	FString Text;
	if (GetString(Section, Key, Text, Filename))
	{
		Value = FCString::Atoi64(*Text);
		return true;
	}
	return false;
}
bool FConfigCacheIni::GetFloat
(
	const TCHAR*		Section,
	const TCHAR*		Key,
	float&				Value,
	const FString&	Filename
)
{
	FString Text; 
	if( GetString( Section, Key, Text, Filename ) )
	{
		Value = FCString::Atof(*Text);
		return true;
	}
	return false;
}
bool FConfigCacheIni::GetDouble
	(
	const TCHAR*		Section,
	const TCHAR*		Key,
	double&				Value,
	const FString&	Filename
	)
{
	FString Text; 
	if( GetString( Section, Key, Text, Filename ) )
	{
		Value = FCString::Atod(*Text);
		return true;
	}
	return false;
}
bool FConfigCacheIni::GetBool
(
	const TCHAR*		Section,
	const TCHAR*		Key,
	bool&				Value,
	const FString&	Filename
)
{
	FString Text; 
	if( GetString( Section, Key, Text, Filename ) )
	{
		Value = FCString::ToBool(*Text);
		return true;
	}
	return false;
}
int32 FConfigCacheIni::GetArray
(
	const TCHAR*		Section,
	const TCHAR*		Key,
	TArray<FString>&	out_Arr,
	const FString&	Filename
)
{
	FRemoteConfig::Get()->FinishRead(*Filename); // Ensure the remote file has been loaded and processed
	out_Arr.Empty();
	FConfigFile* File = Find(Filename);
	if ( File != nullptr )
	{
		File->GetArray(Section, Key, out_Arr);
	}

	if (out_Arr.Num())
	{
		FCoreDelegates::TSOnConfigValueRead().Broadcast(*Filename, Section, Key);
	}

	return out_Arr.Num();
}
/** Loads a "delimited" list of string
 * @param Section - Section of the ini file to load from
 * @param Key - The key in the section of the ini file to load
 * @param out_Arr - Array to load into
 * @param Delimiter - Break in the strings
 * @param Filename - Ini file to load from
 */
int32 FConfigCacheIni::GetSingleLineArray
(
	const TCHAR*		Section,
	const TCHAR*		Key,
	TArray<FString>&	out_Arr,
	const FString&	Filename
)
{
	FString FullString;
	bool bValueExisted = GetString(Section, Key, FullString, Filename);
	const TCHAR* RawString = *FullString;

	//tokenize the string into out_Arr
	FString NextToken;
	while ( FParse::Token(RawString, NextToken, false) )
	{
		out_Arr.Add(MoveTemp(NextToken));
	}
	return bValueExisted;
}

bool FConfigCacheIni::GetColor
(
 const TCHAR*		Section,
 const TCHAR*		Key,
 FColor&			Value,
 const FString&	Filename
 )
{
	FString Text; 
	if( GetString( Section, Key, Text, Filename ) )
	{
		return Value.InitFromString(Text);
	}
	return false;
}

bool FConfigCacheIni::GetVector2D(
	const TCHAR*   Section,
	const TCHAR*   Key,
	FVector2D&     Value,
	const FString& Filename)
{
	FString Text;
	if (GetString(Section, Key, Text, Filename))
	{
		return Value.InitFromString(Text);
	}
	return false;
}


bool FConfigCacheIni::GetVector
(
 const TCHAR*		Section,
 const TCHAR*		Key,
 FVector&			Value,
 const FString&	Filename
 )
{
	FString Text; 
	if( GetString( Section, Key, Text, Filename ) )
	{
		return Value.InitFromString(Text);
	}
	return false;
}

bool FConfigCacheIni::GetVector4
(
 const TCHAR*		Section,
 const TCHAR*		Key,
 FVector4&			Value,
 const FString&	Filename
)
{
	FString Text;
	if(GetString(Section, Key, Text, Filename))
	{
		return Value.InitFromString(Text);
	}
	return false;
}

bool FConfigCacheIni::GetRotator
(
 const TCHAR*		Section,
 const TCHAR*		Key,
 FRotator&			Value,
 const FString&	Filename
 )
{
	FString Text; 
	if( GetString( Section, Key, Text, Filename ) )
	{
		return Value.InitFromString(Text);
	}
	return false;
}

void FConfigCacheIni::SetInt
(
	const TCHAR*	Section,
	const TCHAR*	Key,
	int32				Value,
	const FString&	Filename
)
{
	TCHAR Text[MAX_SPRINTF];
	FCString::Sprintf( Text, TEXT("%i"), Value );
	SetString( Section, Key, Text, Filename );
}
void FConfigCacheIni::SetFloat
(
	const TCHAR*		Section,
	const TCHAR*		Key,
	float				Value,
	const FString&	Filename
)
{
	FConfigFile* File = Find(Filename);
	if (!File)
	{
		return;
	}

	File->SetFloat(Section, Key, Value);
}
void FConfigCacheIni::SetDouble
(
	const TCHAR*		Section,
	const TCHAR*		Key,
	double				Value,
	const FString&	Filename
)
{
	FConfigFile* File = Find(Filename);
	if (!File)
	{
		return;
	}

	File->SetDouble(Section, Key, Value);
}
void FConfigCacheIni::SetBool
(
	const TCHAR*		Section,
	const TCHAR*		Key,
	bool				Value,
	const FString&	Filename
)
{
	FConfigFile* File = Find(Filename);
	if (!File)
	{
		return;
	}

	File->SetBool(Section, Key, Value);
}

void FConfigCacheIni::SetArray
(
	const TCHAR*			Section,
	const TCHAR*			Key,
	const TArray<FString>&	Value,
	const FString&		Filename
)
{
	FConfigFile* File = Find(Filename);
	if (!File)
	{
		return;
	}

	File->SetArray(Section, Key, Value);
}
/** Saves a "delimited" list of strings
 * @param Section - Section of the ini file to save to
 * @param Key - The key in the section of the ini file to save
 * @param In_Arr - Array to save from
 * @param Filename - Ini file to save to
 */
void FConfigCacheIni::SetSingleLineArray
(
	const TCHAR*			Section,
	const TCHAR*			Key,
	const TArray<FString>&	In_Arr,
	const FString&		Filename
)
{
	FString FullString;

	//append all strings to single string
	for (int32 i = 0; i < In_Arr.Num(); ++i)
	{
		FullString += In_Arr[i];
		FullString += TEXT(" ");
	}

	//save to ini file
	SetString(Section, Key, *FullString, Filename);
}

void FConfigCacheIni::SetColor
(
 const TCHAR*		Section,
 const TCHAR*		Key,
 const FColor		Value,
 const FString&	Filename
 )
{
	SetString( Section, Key, *Value.ToString(), Filename );
}

void FConfigCacheIni::SetVector2D(
	const TCHAR*   Section,
	const TCHAR*   Key,
	FVector2D      Value,
	const FString& Filename)
{
	SetString(Section, Key, *Value.ToString(), Filename);
}

void FConfigCacheIni::SetVector
(
 const TCHAR*		Section,
 const TCHAR*		Key,
 const FVector		 Value,
 const FString&	Filename
 )
{
	SetString( Section, Key, *Value.ToString(), Filename );
}

void FConfigCacheIni::SetVector4
(
 const TCHAR*		Section,
 const TCHAR*		Key,
 const FVector4&	 Value,
 const FString&	Filename
)
{
	SetString(Section, Key, *Value.ToString(), Filename);
}

void FConfigCacheIni::SetRotator
(
 const TCHAR*		Section,
 const TCHAR*		Key,
 const FRotator		Value,
 const FString&	Filename
 )
{
	SetString( Section, Key, *Value.ToString(), Filename );
}



static FConfigCommandStream* FindStaticLayer(FConfigCacheIni* ConfigSystem, const FString& BranchFilename, const FString& LayerKeySubstring, FString& OutFilename, FConfigBranch*& OutBranch)
{
	OutBranch = ConfigSystem->FindBranch(*BranchFilename, BranchFilename);
	if (OutBranch)
	{
		for (TPair<FString, FConfigCommandStream>& Pair : OutBranch->StaticLayers)
		{
			if (Pair.Key.Contains(LayerKeySubstring))
			{
				OutFilename = Pair.Key;
				return &Pair.Value;
			}
		}

		TUtf8StringBuilder<256> LayerKeySubstringUtf8(InPlace, LayerKeySubstring);

		// we didn't find an existing layer, so look to see if we have a matching entry in the hierarchy and insert an empty layer for it
		for (TPair<int32, FUtf8String>& IniFile : OutBranch->Hierarchy)
		{
			if (IniFile.Value.Contains(LayerKeySubstringUtf8))
			{
				int KeyToInsert = IniFile.Key;
				FString FilenameToInsert = FString(IniFile.Value);
				bool bInserted = false;

				// now for the painful process of finding where to insert it into the static layers - and since we need it ordered
				// we have to make an entire new map
				TMap<FString, FConfigCommandStream> NewLayers;
				FConfigCommandStream* NewLayer = nullptr;
				for (TPair<FString, FConfigCommandStream>& Pair : OutBranch->StaticLayers)
				{
					// find the entry in the hierarchy that matches this layer filename
					if (const int32* LayerKey = OutBranch->Hierarchy.FindKey(FUtf8String(Pair.Key)))
					{
						if (!bInserted && *LayerKey > KeyToInsert)
						{
							NewLayer = &NewLayers.Add(FilenameToInsert, FConfigCommandStream());
							bInserted = true;
						}
						NewLayers.Add(Pair.Key, Pair.Value);
					}
				}

				if (bInserted)
				{
					OutBranch->StaticLayers = NewLayers;
					OutFilename = MoveTemp(FilenameToInsert);
					return NewLayer;
				}
			}
		}
	}
	return nullptr;
}

/**
 * A wrapper function that finds a layer by name substring and section, calls a passed-in lambda to actually modify the section, writes
 * changes to disk, then recalculates the in-memory config values
 */
static bool ModifyAndWriteLayer(FConfigCacheIni* ConfigSystem, const FString& SectionName, FName Key, const FString& BranchName, const FString& LayerSubstring,
	bool bWriteFile, TFunction<void(const FConfigBranch*, FConfigCommandStreamSection*)> Modify)
{
	// if reapplying ends being slow, we will need to make this an option
	const bool bReapplyLayers = true;

	FString FilenameToWrite;
	FConfigBranch* Branch;
	if (FConfigCommandStream* Layer = FindStaticLayer(ConfigSystem, BranchName, LayerSubstring, FilenameToWrite, Branch))
	{
		FConfigCommandStreamSection* Sec = Layer->FindOrAddSectionInternal(SectionName);
		Modify(Branch, Sec);

		if (bWriteFile)
		{
			FSinglePropertyConfigHelper Writer(FilenameToWrite, SectionName, Key.ToString(), Sec);
			Writer.UpdateConfigFile();
		}

		if (bReapplyLayers)
		{
			Branch->ReapplyLayers();
		}

		return true;
	}

	return false;
}


bool FConfigCacheIni::SetInSection(const TCHAR* Section, FName Key, const FString& Value, const FString& Filename)
{
	if (FConfigFile* File = Find(*Filename))
	{
		return File->SetInSection(Section, Key, Value);
	}
	return false;
}

bool FConfigCacheIni::SetInSectionOfStaticLayer(const TCHAR* Section, FName Key, const FString& Value, const FString& BranchName, const FString& LayerSubstring, bool bWriteFile)
{
	return ModifyAndWriteLayer(this, Section, Key, BranchName, LayerSubstring, bWriteFile, 
		[Value, Key](const FConfigBranch* Branch, FConfigCommandStreamSection* Sec)
		{
			// look for an existing value to replace (to maintain order)
			TArray<FConfigValue*> ExistingValues;
			Sec->MultiFindPointer(Key, ExistingValues);
			if (ExistingValues.Num() == 1)
			{
				*ExistingValues[0] = FConfigValue(Value, FConfigValue::EValueType::Set);
			}
			// otherwise remove any that are there, and add to the end (we don't try to maintain
			// order in the rare case there is an array and we are replacing with a single)
			else
			{
				Sec->Remove(Key);
				Sec->Add(Key, FConfigValue(Value, FConfigValue::EValueType::Set));
			}
		});
}


bool FConfigCacheIni::AddToSection(const TCHAR* Section, FName Key, const FString& Value, const FString& Filename)
{
	if (FConfigFile* File = Find(*Filename))
	{
		return File->AddToSection(Section, Key, Value);
	}
	return false;
}

bool FConfigCacheIni::AddToSectionOfStaticLayer(const TCHAR* Section, FName Key, const FString& Value, const FString& BranchName, const FString& LayerSubstring, bool bWriteFile)
{
	return ModifyAndWriteLayer(this, Section, Key, BranchName, LayerSubstring, bWriteFile,
		[Value, Key](const FConfigBranch* Branch, FConfigCommandStreamSection* Sec)
		{
			Sec->Add(Key, FConfigValue(Value, FConfigValue::EValueType::ArrayAdd));
		});
}

bool FConfigCacheIni::AddUniqueToSection(const TCHAR* Section, FName Key, const FString& Value, const FString& Filename)
{
	if (FConfigFile* File = Find(*Filename))
	{
		return File->AddUniqueToSection(Section, Key, Value);
	}
	return false;
}

bool FConfigCacheIni::AddUniqueToSectionOfStaticLayer(const TCHAR* Section, FName Key, const FString& Value, const FString& BranchName, const FString& LayerSubstring, bool bWriteFile)
{
	return ModifyAndWriteLayer(this, Section, Key, BranchName, LayerSubstring, bWriteFile,
		[Value, Key](const FConfigBranch* Branch, FConfigCommandStreamSection* Sec)
		{
			Sec->Add(Key, FConfigValue(Value, FConfigValue::EValueType::ArrayAddUnique));
		});
}

bool FConfigCacheIni::RemoveKeyFromSection(const TCHAR* Section, FName Key, const FString& Filename)
{
	if (FConfigFile* File = Find(*Filename))
	{
		return File->RemoveKeyFromSection(Section, Key);
	}
	return false;
}

bool FConfigCacheIni::RemoveKeyFromSectionOfStaticLayer(const TCHAR* Section, FName Key, const FString& BranchName, const FString& LayerSubstring, bool bWriteFile)
{
	return ModifyAndWriteLayer(this, Section, Key, BranchName, LayerSubstring, bWriteFile,
		[Key](const FConfigBranch* Branch, FConfigCommandStreamSection* Sec)
		{
			Sec->Remove(Key);
			Sec->Add(Key, FConfigValue(TEXT("__Clear__"), FConfigValue::EValueType::Clear));
		});
}

bool FConfigCacheIni::RemoveFromSection(const TCHAR* Section, FName Key, const FString& Value, const FString& Filename)
{
	if (FConfigFile* File = Find(*Filename))
	{
		return File->RemoveFromSection(Section, Key, Value);
	}
	return false;
}

bool FConfigCacheIni::RemoveFromSectionOfStaticLayer(const TCHAR* Section, FName Key, const FString& Value, const FString& BranchName, const FString& LayerSubstring, bool bWriteFile)
{
	return ModifyAndWriteLayer(this, Section, Key, BranchName, LayerSubstring, bWriteFile,
		[Key, Value](const FConfigBranch* Branch, FConfigCommandStreamSection* Sec)
		{
			Sec->Remove(Key);
			Sec->Add(Key, FConfigValue(Value, FConfigValue::EValueType::Remove));
		});
}

bool FConfigCacheIni::ResetKeyInSection(const TCHAR* Section, FName Key, const FString& Filename)
{
	if (FConfigFile* File = Find(*Filename))
	{
		return File->ResetKeyInSection(Section, Key);
	}
	return false;
}

bool FConfigCacheIni::ResetKeyInSectionOfStaticLayer(const TCHAR* Section, FName Key, const FString& BranchName, const FString& LayerSubstring, bool bWriteFile)
{
	return ModifyAndWriteLayer(this, Section, Key, BranchName, LayerSubstring, bWriteFile,
		[Key](const FConfigBranch* Branch, FConfigCommandStreamSection* Sec)
		{
			Sec->Remove(Key);
		});
}

bool FConfigCacheIni::SetArrayInSectionOfStaticLayer(const TCHAR* Section, FName Key, const TArray<FString>& Values, bool bClearArray, const FString& BranchName, const FString& LayerSubstring, bool bWriteFile)
{
	return ModifyAndWriteLayer(this, Section, Key, BranchName, LayerSubstring, bWriteFile,
		[Section, Key, Values, bClearArray, LayerSubstring](const FConfigBranch* Branch, FConfigCommandStreamSection* Sec)
		{
			// we always start this section over 
			Sec->Remove(Key);
			if (bClearArray)
			{
				Sec->Add(Key, FConfigValue(TEXT("___Clear___"), FConfigValue::EValueType::Clear));
				for (const FString& Value : Values)
				{
					Sec->Add(Key, FConfigValue(Value, FConfigValue::EValueType::ArrayAddUnique));
				}
			}
			else
			{
				// look one layer down, so we can minimize the operations needed to get tot he passed in array
				FConfigFile ValuesBeforeLayer;
				Branch->MergeStaticLayersUpTo(LayerSubstring, ValuesBeforeLayer);
				TArray<const FConfigValue*> ParentValues;
				if (const FConfigSection* ParentSection = ValuesBeforeLayer.FindSection(Section))
				{
					ParentSection->MultiFindPointer(Key, ParentValues, true);
				}

				TArray<FString> ValuesToAdd;
				for (const FString& Value : Values)
				{
					bool bFound = false;
					for (const FConfigValue* ParentValue : ParentValues)
					{
						if (ParentValue->GetValueForWriting() == Value)
						{
							ParentValues.Remove(ParentValue);
							bFound = true;
							break;
						}
					}
					if (!bFound)
					{
						ValuesToAdd.Add(Value);
					}
				}

				// any remaining existing value needs to be removed
				// @todo: this needs testing with expanion variables - we probably had unexpanded values coming in, in which case we never want to expand - hard to say
				for (const FConfigValue* ParentValue : ParentValues)
				{
					Sec->Add(Key, FConfigValue(ParentValue->GetSavedValueForWriting(), FConfigValue::EValueType::Remove));
				}
				for (const FString& Value : ValuesToAdd)
				{
					Sec->Add(Key, FConfigValue(Value, FConfigValue::EValueType::ArrayAddUnique));
				}
			}
		});
}



/**
 * Archive for counting config file memory usage.
 */
class FArchiveCountConfigMem : public FArchive
{
public:
	FArchiveCountConfigMem()
	:	Num(0)
	,	Max(0)
	{
		ArIsCountingMemory = true;
	}
	SIZE_T GetNum()
	{
		return Num;
	}
	SIZE_T GetMax()
	{
		return Max;
	}
	void CountBytes( SIZE_T InNum, SIZE_T InMax )
	{
		Num += InNum;
		Max += InMax;
	}
protected:
	SIZE_T Num, Max;
};

struct FDetailedConfigMemUsage : public FArchiveCountConfigMem
{
	TMap<FString, FArchiveCountConfigMem> PerLayerInfo;
	TMap<FString, FArchiveCountConfigMem> PerSectionInfo;
	TMap<FString, FArchiveCountConfigMem> PerSectionValueInfo;

	FDetailedConfigMemUsage(FConfigBranch* Branch, bool bTrackDetails)
	{
		(*this) << *Branch;

		if (bTrackDetails)
		{
			Branch->RunOnEachFile([this](FConfigFile& File, const FString& Name)
			{
				TrackFile(Name, File);
			});

			Branch->RunOnEachCommandStream([this](FConfigCommandStream& Stream, const FString& Name)
			{
				TrackCommandStream(Name, Stream);
			});
		}
	}

private:
	void TrackFile(const FString& Name, FConfigFile& File)
	{
		FArchiveCountConfigMem& Ar = PerLayerInfo.FindOrAdd(Name);
		Ar << File;

		for (const TPair<FString, FConfigSection>& Pair: AsConst(File))
		{
			FArchiveCountConfigMem& SectionAr = PerSectionInfo.FindOrAdd(Pair.Key);
			SectionAr << const_cast<FConfigSection&>(Pair.Value);

			FArchiveCountConfigMem& ValueAr = PerSectionValueInfo.FindOrAdd(Pair.Key);
			for (const TPair<FName, FConfigValue>& Pair2 : Pair.Value)
			{
				ValueAr << const_cast<FConfigValue&>(Pair2.Value);
			}
		}
	}

	void TrackCommandStream(const FString& Name, FConfigCommandStream& Stream)
	{
		FArchiveCountConfigMem& Ar = PerLayerInfo.FindOrAdd(Name);
		Ar << Stream;

		for (auto& Pair : Stream)
		{
			FArchiveCountConfigMem& SectionAr = const_cast<FArchiveCountConfigMem&>(PerSectionInfo.FindOrAdd(Pair.Key));
			SectionAr << Pair.Value;

			FArchiveCountConfigMem& ValueAr = PerSectionValueInfo.FindOrAdd(Pair.Key);
			for (const TPair<FName, FConfigValue>& Pair2 : Pair.Value)
			{
				ValueAr << const_cast<FConfigValue&>(Pair2.Value);
			}
		}
	}
};



/**
 * Tracks the amount of memory used by a single config or loc file
 */
struct FConfigFileMemoryData
{
	FString	ConfigFilename;
	SIZE_T		CurrentSize;
	SIZE_T		MaxSize;

	FConfigFileMemoryData( const FString& InFilename, SIZE_T InSize, SIZE_T InMax )
	: ConfigFilename(InFilename), CurrentSize(InSize), MaxSize(InMax)
	{}
};

/**
 * Tracks the memory data recorded for all loaded config files.
 */
struct FConfigMemoryData
{
	int32 NameIndent;
	int32 SizeIndent;
	int32 MaxSizeIndent;

	TArray<FConfigFileMemoryData> MemoryData;

	FConfigMemoryData()
	: NameIndent(0), SizeIndent(0), MaxSizeIndent(0)
	{}

	void AddConfigFile( const FString& ConfigFilename, FArchiveCountConfigMem& MemAr )
	{
		SIZE_T TotalMem = MemAr.GetNum();
		SIZE_T MaxMem = MemAr.GetMax();

		NameIndent = FMath::Max(NameIndent, ConfigFilename.Len());
		SizeIndent = FMath::Max(SizeIndent, FString::FromInt((int32)TotalMem).Len());
		MaxSizeIndent = FMath::Max(MaxSizeIndent, FString::FromInt((int32)MaxMem).Len());
		
		MemoryData.Emplace( ConfigFilename, TotalMem, MaxMem );
	}

	void SortBySize()
	{
		struct FCompareFConfigFileMemoryData
		{
			FORCEINLINE bool operator()( const FConfigFileMemoryData& A, const FConfigFileMemoryData& B ) const
			{
				return ( B.CurrentSize == A.CurrentSize ) ? ( B.MaxSize < A.MaxSize ) : ( B.CurrentSize < A.CurrentSize );
			}
		};
		MemoryData.Sort( FCompareFConfigFileMemoryData() );
	}
};

/**
 * Dumps memory stats for each file in the config cache to the specified archive.
 *
 * @param	Ar	the output device to dump the results to
 */
void FConfigCacheIni::ShowMemoryUsage( FOutputDevice& Ar )
{
	UE::TScopeLock Lock(OtherFilesLock);
	FConfigMemoryData ConfigCacheMemoryData;

	for (TPair<FString, FConfigBranch*>& Pair : OtherFiles)
	{
		FString Filename = Pair.Key;
		FConfigBranch& ConfigBranch = *Pair.Value;

		FArchiveCountConfigMem MemAr;

		// count the bytes used for storing the filename
		MemAr << Filename;

		// count the bytes used for storing the array of SectionName->Section pairs
		MemAr << ConfigBranch;
		
		ConfigCacheMemoryData.AddConfigFile(Filename, MemAr);
	}
	{
		FArchiveCountConfigMem MemAr;
		MemAr << KnownFiles;
		ConfigCacheMemoryData.AddConfigFile(TEXT("KnownFiles"), MemAr);
	}

	// add a little extra spacing between the columns
	ConfigCacheMemoryData.SizeIndent += 10;
	ConfigCacheMemoryData.MaxSizeIndent += 10;

	// record the memory used by the FConfigCacheIni's TMap
	FArchiveCountConfigMem MemAr;
	OtherFiles.CountBytes(MemAr);
	OtherFileNames.CountBytes(MemAr);

	SIZE_T TotalMemoryUsage=MemAr.GetNum();
	SIZE_T MaxMemoryUsage=MemAr.GetMax();

	Ar.Log(TEXT("Config cache memory usage:"));
	// print out the header
	Ar.Logf(TEXT("%*s %*s %*s"), ConfigCacheMemoryData.NameIndent, TEXT("FileName"), ConfigCacheMemoryData.SizeIndent, TEXT("NumBytes"), ConfigCacheMemoryData.MaxSizeIndent, TEXT("MaxBytes"));

	ConfigCacheMemoryData.SortBySize();
	for ( int32 Index = 0; Index < ConfigCacheMemoryData.MemoryData.Num(); Index++ )
	{
		FConfigFileMemoryData& ConfigFileMemoryData = ConfigCacheMemoryData.MemoryData[Index];
			Ar.Logf(TEXT("%*s %*u %*u"),
			ConfigCacheMemoryData.NameIndent, *ConfigFileMemoryData.ConfigFilename,
			ConfigCacheMemoryData.SizeIndent, (uint32)ConfigFileMemoryData.CurrentSize,
			ConfigCacheMemoryData.MaxSizeIndent, (uint32)ConfigFileMemoryData.MaxSize);

		TotalMemoryUsage += ConfigFileMemoryData.CurrentSize;
		MaxMemoryUsage += ConfigFileMemoryData.MaxSize;
	}

	Ar.Logf(TEXT("%*s %*u %*u"),
		ConfigCacheMemoryData.NameIndent, TEXT("Total"),
		ConfigCacheMemoryData.SizeIndent, (uint32)TotalMemoryUsage,
		ConfigCacheMemoryData.MaxSizeIndent, (uint32)MaxMemoryUsage);
}



SIZE_T FConfigCacheIni::GetMaxMemoryUsage()
{
	UE::TScopeLock Lock(OtherFilesLock);

	// record the memory used by the FConfigCacheIni's TMap
	FArchiveCountConfigMem MemAr;
	OtherFiles.CountBytes(MemAr);
	OtherFileNames.CountBytes(MemAr);

	SIZE_T TotalMemoryUsage=MemAr.GetNum();
	SIZE_T MaxMemoryUsage=MemAr.GetMax();


	FConfigMemoryData ConfigCacheMemoryData;

	for (TPair<FString, FConfigBranch*>& Pair : OtherFiles)
	{
		FString Filename = Pair.Key;
		FConfigFile& ConfigFile = Pair.Value->InMemoryFile;

		FArchiveCountConfigMem FileMemAr;

		// count the bytes used for storing the filename
		FileMemAr << Filename;

		// count the bytes used for storing the array of SectionName->Section pairs
		FileMemAr << ConfigFile;

		ConfigCacheMemoryData.AddConfigFile(Filename, FileMemAr);
	}
	{
		FArchiveCountConfigMem FileMemAr;
		FileMemAr << KnownFiles;
		ConfigCacheMemoryData.AddConfigFile(TEXT("KnownFiles"), FileMemAr);
	}

	for ( int32 Index = 0; Index < ConfigCacheMemoryData.MemoryData.Num(); Index++ )
	{
		FConfigFileMemoryData& ConfigFileMemoryData = ConfigCacheMemoryData.MemoryData[Index];

		TotalMemoryUsage += ConfigFileMemoryData.CurrentSize;
		MaxMemoryUsage += ConfigFileMemoryData.MaxSize;
	}

	return MaxMemoryUsage;
}

bool FConfigCacheIni::ForEachEntry(const FKeyValueSink& Visitor, const TCHAR* Section, const FString& Filename)
{
	FConfigFile* File = Find(Filename);
	if(!File)
	{
		return false;
	}

	const FConfigSection* Sec = File->FindSection(Section);
	if(!Sec)
	{
		return false;
	}

	for(FConfigSectionMap::TConstIterator It(*Sec); It; ++It)
	{
		Visitor.Execute(*It.Key().GetPlainNameString(), *It.Value().GetValue());
	}

	return true;
}

FString FConfigCacheIni::GetDestIniFilename(const TCHAR* BaseIniName, const TCHAR* PlatformName, const TCHAR* GeneratedConfigDir)
{
	// figure out what to look for on the commandline for an override
	FString CommandLineSwitch = FString::Printf(TEXT("%sINI="), BaseIniName);
	
	// if it's not found on the commandline, then generate it
	FString IniFilename;
	if (FParse::Value(FCommandLine::Get(), *CommandLineSwitch, IniFilename) == false)
	{
		FString Name(PlatformName ? PlatformName : ANSI_TO_TCHAR(FPlatformProperties::PlatformName()));

		// if the BaseIniName doesn't start with the config dir, put it all together
		if (FString(BaseIniName).StartsWith(GeneratedConfigDir) && FPaths::GetExtension(BaseIniName) == TEXT("ini"))
		{
			IniFilename = BaseIniName;
		}
		else
		{
			IniFilename = FString::Printf(TEXT("%s%s/%s.ini"), GeneratedConfigDir, *Name, BaseIniName);
		}
	}

	// standardize it!
	FPaths::MakeStandardFilename(IniFilename);
	return IniFilename;
}

void FConfigCacheIni::SaveCurrentStateForBootstrap(const TCHAR* Filename)
{
	TArray<uint8> FileContent;
	{
		// Use FMemoryWriter because FileManager::CreateFileWriter doesn't serialize FName as string and is not overridable
		FMemoryWriter MemoryWriter(FileContent, true);
		SerializeStateForBootstrap_Impl(MemoryWriter);
	}

	FFileHelper::SaveArrayToFile(FileContent, Filename);
}

void FConfigCacheIni::Serialize(FArchive& Ar)
{
	if (Ar.IsLoading())
	{
		int Num;
		Ar << Num;
		for (int Index = 0; Index < Num; Index++)
		{
			FString Filename;
			FConfigBranch* Branch = new FConfigBranch;
			Ar << Filename;
			Ar << *Branch;
			OtherFiles.Add(Filename, Branch);
			OtherFileNames.Add(Filename);
		}
	}
	else
	{
		int Num = OtherFiles.Num();
		Ar << Num;
		for (TPair<FString, FConfigBranch*>& It : OtherFiles)
		{
			Ar << It.Key;
			Ar << *It.Value;
		}
	}
	Ar << KnownFiles;
	Ar << bAreFileOperationsDisabled;
	Ar << bIsReadyForUse;
	Ar << Type;
	Ar << PlatformName;
	Ar << StagedPluginConfigCache;

	bool bHasGlobalCache = StagedGlobalConfigCache != nullptr;
	Ar << bHasGlobalCache;
	if (bHasGlobalCache)
	{
		if (Ar.IsLoading()) StagedGlobalConfigCache = new TSet<FString>();
		// checking for null for SA validation
		if (StagedGlobalConfigCache != nullptr)
		{
			Ar << *StagedGlobalConfigCache;
		}
	}
}

void FConfigCacheIni::SerializeStateForBootstrap_Impl(FArchive& Ar)
{
	// This implementation is meant to stay private and be used for
	// bootstrapping another processes' config cache with a serialized state.
	// It doesn't include any versioning as it is used with the
	// the same binary executable for both the parent and
	// children processes. It also takes care of saving/restoring
	// global ini variables.
	Serialize(Ar);
	Ar << GEditorIni;
	Ar << GEditorKeyBindingsIni;
	Ar << GEditorLayoutIni;
	Ar << GEditorSettingsIni;
	Ar << GEditorPerProjectIni;
	Ar << GCompatIni;
	Ar << GLightmassIni;
	Ar << GScalabilityIni;
	Ar << GHardwareIni;
	Ar << GInputIni;
	Ar << GGameIni;
	Ar << GGameUserSettingsIni;
	Ar << GRuntimeOptionsIni;
	Ar << GEngineIni;
}


bool FConfigCacheIni::InitializeKnownConfigFiles(FConfigContext& Context)
{
	// check for scalability platform override.
	FConfigContext* ScalabilityPlatformOverrideContext = nullptr;
#if WITH_EDITOR || IS_PROGRAM
	{
		FString ScalabilityPlatformOverrideCommandLine;
		FParse::Value(FCommandLine::Get(), TEXT("ScalabilityIniPlatformOverride="), ScalabilityPlatformOverrideCommandLine);
		if (!ScalabilityPlatformOverrideCommandLine.IsEmpty())
		{
			ScalabilityPlatformOverrideContext = new FConfigContext(FConfigContext::ReadIntoConfigSystem(Context.ConfigSystem, ScalabilityPlatformOverrideCommandLine));
		}
	}
#endif

	bool bEngineConfigCreated = false;
	for (uint8 KnownIndex = 0; KnownIndex < (uint8)EKnownIniFile::NumKnownFiles; KnownIndex++)
	{
		FConfigBranch& KnownBranch = Context.ConfigSystem->KnownFiles.Branches[KnownIndex];

#if UE_WITH_CONFIG_TRACKING
		// We cannot set KnownFiles' LoadType in the FConfigCacheIni constructor because we need to compare with GConfig,
		// which is not set during GConfig's constructor. We have to set it before calling Load on the ConfigFile, since
		// Load can read values and LoadType must be set before any values are read.
		UE::ConfigAccessTracking::FFile* FileAccess = KnownBranch.InMemoryFile.GetFileAccess();
		if (FileAccess)
		{
			FileAccess->SetAsLoadTypeConfigSystem(*Context.ConfigSystem, KnownBranch.InMemoryFile);
			FileAccess->OverrideFilenameToLoad = KnownBranch.IniName;
		}
#endif

		// allow for scalability to come from another platform (made above)
		FConfigContext& ContextToUse = (KnownIndex == (uint8)EKnownIniFile::Scalability && ScalabilityPlatformOverrideContext) ? *ScalabilityPlatformOverrideContext : Context;

		// and load it, saving the dest path to IniPath
		bool bConfigCreated = ContextToUse.Load(*KnownBranch.IniName.ToString(), KnownBranch.IniPath);
		
		// we want to return if the Engine config was successfully created (to not remove any functionality from old code)
		if (KnownIndex == (uint8)EKnownIniFile::Engine)
		{
			bEngineConfigCreated = bConfigCreated;
		}
	}

	// GConfig set itself ready for use later on
	if (Context.ConfigSystem != GConfig)
	{
		Context.ConfigSystem->bIsReadyForUse = true;
	}

	return bEngineConfigCreated;
}

bool FConfigCacheIni::IsKnownConfigName(FName ConfigName)
{
	return KnownFiles.GetFile(ConfigName) != nullptr;
}

const FConfigFile* FConfigCacheIni::FKnownConfigFiles::GetFile(FName Name)
{
	// sharing logic
	return GetMutableFile(Name);
}

FConfigFile* FConfigCacheIni::FKnownConfigFiles::GetMutableFile(FName Name)
{
	FConfigBranch* Branch = GetBranch(Name);
	return Branch ? &Branch->InMemoryFile : nullptr;
}

FConfigBranch* FConfigCacheIni::FKnownConfigFiles::GetBranch(FName Name)
{
	// walk the list of files looking for matching FName (a TMap was a bit slower)
	for (FConfigBranch& Branch : Branches)
	{
		if (Branch.IniName == Name)
		{
			return &Branch;
		}
	}
	return nullptr;
}

const FString& FConfigCacheIni::FKnownConfigFiles::GetFilename(FName Name)
{
	static FString Empty;
	const FConfigBranch* Branch = GetBranch(Name);
	return Branch ? Branch->IniPath : Empty;
}

FConfigCacheIni::FKnownConfigFiles::FKnownConfigFiles()
{
	// set the FNames associated with each file

	// 	Files[(uint8)EKnownIniFile::Engine].IniName = FName("Engine");
	#define SET_KNOWN_NAME(Ini) Branches[(uint8)EKnownIniFile::Ini].IniName = FName(#Ini);
		ENUMERATE_KNOWN_INI_FILES(SET_KNOWN_NAME);
	#undef SET_KNOWN_NAME
}

FArchive& operator<<(FArchive& Ar, FConfigCacheIni::FKnownConfigFiles& KnownFiles)
{
	for (FConfigBranch& Branch : KnownFiles.Branches)
	{
		Ar << Branch;
	}

	return Ar;
}

FArchive& operator<<(FArchive& Ar, FConfigBranch& ConfigBranch)
{
	Ar << ConfigBranch.bIsHierarchical;
	Ar << ConfigBranch.InMemoryFile;
	Ar << ConfigBranch.Hierarchy;
	Ar << ConfigBranch.CombinedStaticLayers;
	Ar << ConfigBranch.FinalCombinedLayers;
	Ar << ConfigBranch.IniName;
	Ar << ConfigBranch.IniPath;

	// needed to count full memory usage
	if (!Ar.IsPersistent())
	{
		Ar << ConfigBranch.Platform;
		Ar << ConfigBranch.SourceEngineConfigDir;
		Ar << ConfigBranch.SourceProjectConfigDir;
		Ar << ConfigBranch.StaticLayers;
		Ar << ConfigBranch.SavedLayer;
		Ar << ConfigBranch.CommandLineOverrides;
		Ar << ConfigBranch.RuntimeChanges;
		for (FConfigBranch::DynamicLayerList::TIterator Node(ConfigBranch.DynamicLayers.GetHead()); Node; ++Node)
		{
			FConfigCommandStream& S = *Node.GetNode()->GetValue();
			Ar << S;
		}
	}
	return Ar;
}


#if PRELOAD_BINARY_CONFIG

#include "Misc/PreLoadFile.h"
static FPreLoadFile GPreLoadConfigBin(TEXT("{PROJECT}Config/BinaryConfig.ini"));

bool FConfigCacheIni::CreateGConfigFromSaved(const TCHAR* Filename)
{
	SCOPED_BOOT_TIMING("FConfigCacheIni::CreateGConfigFromSaved");
	// get the already loaded file
	int64 Size;
	void* PreloadedData = GPreLoadConfigBin.TakeOwnershipOfLoadedData(&Size);

	FString BinaryConfigScalabilityOverride;
	FParse::Value(FCommandLine::Get(), TEXT("BinaryConfigScalabilityOverride="), BinaryConfigScalabilityOverride);
	bool bIsConfigScalabilityOverride = !BinaryConfigScalabilityOverride.IsEmpty();
	TArray<uint8> FileContent;
	if (bIsConfigScalabilityOverride)
	{
		FString BinaryConfig = FString::Printf(TEXT("%sBinaryConfig%s.ini"), *FPaths::ProjectConfigDir(), *BinaryConfigScalabilityOverride);
		if (FFileHelper::LoadFileToArray(FileContent, *BinaryConfig, FILEREAD_Silent))
		{
			// Free immediately the default Binary Config preloaded Data
			if (PreloadedData != nullptr)
			{
				FMemory::Free(PreloadedData);
			}

			// Set the Preload Data to the Override Scalability Binary Config file
			PreloadedData = FileContent.GetData();
			Size = FileContent.Num();
		}
		else
		{
			UE_LOGF(LogConfig, Log, "BinaryConfigScalabilityOverride was set to %ls but BinaryConfig %ls could not be loaded. Reverting to default BinaryConfig.ini file.",
				*BinaryConfigScalabilityOverride, *BinaryConfig);
			bIsConfigScalabilityOverride = false;
		}
	}

	if (PreloadedData == nullptr)
	{
		return false;
	}

	// serialize right out of the preloaded data
	FLargeMemoryReader MemoryReader((uint8*)PreloadedData, Size);
	MemoryReader.SetIsPersistent(true);
	GConfig = new FConfigCacheIni(EConfigCacheType::Temporary, NAME_None /* Platform*/, true /* bInGloballyRegistered */);

	// make an object that we can use to pass to delegates for any extra binary data they want to write
	FCoreDelegates::FExtraBinaryConfigData ExtraData(*GConfig, false);

	GConfig->Serialize(MemoryReader);

	// fix up some things that weren't saved out
	GConfig->Type = EConfigCacheType::DiskBacked;

	// read in any needed generated/saved ini files (using the logic in FConfigContext to determine if it's needed to be loaded)
	// and apply commandline overrides
	FConfigContext Context = FConfigContext::FixupBranchAfterBinaryConfig();
	for (FConfigBranch& Branch : GConfig->KnownFiles.Branches)
	{
		Context.Load(*Branch.IniName.ToString(), /*out*/ Branch.IniPath);
	}

	MemoryReader << ExtraData.Data;

	// now let the delegates pull their data out, after GConfig is set up
	FCoreDelegates::TSAccessExtraBinaryConfigData().Broadcast(ExtraData);

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ConfigReadyForUseBroadcast);
		FCoreDelegates::TSConfigReadyForUse().Broadcast();
	}

	// This Log is likely the first one happening in the engine and will trigger the creation of the log file
	// It must be done after GConfig has been allocated and ready to use to ensure IFileHandle log wont be a 
	// FManagedStorageFileWriteHandle (which might cause dead lock on some platforms when reporting a crash)
	UE_LOGF(LogInit, Display, "Loaded binary GConfig from %lld bytes of data...", Size);
	if (!bIsConfigScalabilityOverride)
	{
		FMemory::Free(PreloadedData);
	}
	return true;
}

#endif

static void LoadRemainingConfigFiles(FConfigContext& Context)
{
	SCOPED_BOOT_TIMING("LoadRemainingConfigFiles");

#if PLATFORM_DESKTOP
	// load some desktop only .ini files
	Context.Load(TEXT("Compat"), GCompatIni);
	Context.Load(TEXT("Lightmass"), GLightmassIni);
#endif

#if WITH_EDITOR
	// load some editor specific .ini files

	Context.Load(TEXT("Editor"), GEditorIni);

	// early loading allows for plugins to append entries in material expressions
	Context.Load(TEXT("MaterialExpressions"));

	// Upgrade editor user settings before loading the editor per project user settings
	FConfigManifest::MigrateEditorUserSettings();
	Context.Load(TEXT("EditorPerProjectUserSettings"), GEditorPerProjectIni);

	// Project agnostic editor ini files, so save them to a shared location (Engine, not Project)
	Context.GeneratedConfigDir = FPaths::EngineEditorSettingsDir();
	Context.Load(TEXT("EditorSettings"), GEditorSettingsIni);
	Context.Load(TEXT("EditorKeyBindings"), GEditorKeyBindingsIni);
	Context.Load(TEXT("EditorLayout"), GEditorLayoutIni);

#endif

	if (FParse::Param(FCommandLine::Get(), TEXT("dumpconfig")))
	{
		GConfig->Dump(*GLog);
	}
}

static void InitializeConfigRemap()
{
	// read in the single remap file
	FConfigFile RemapFile;
	FConfigContext Context = FConfigContext::ReadSingleIntoLocalFile(RemapFile);

#if UE_WITH_CONFIG_TRACKING 
	// Do not report reads of ConfigRemap. The values inside of ConfigRemap permanently affect the operation of
	// ConfigFiles for the rest of the process lifetime, and we cannot handle rereading it for access tracking.
	// TODO: For incremental cooks, we should instead hash RemapFile.ini and add it to a key that invalidates all packages.
	RemapFile.SuppressReporting();
#endif
	
	// read in engine and project ini files (these are not hierarchical, so it has to be done in two passes)
	for (int Pass = 0; Pass < 2; Pass++)
	{
		// if there isn't an active project, then skip the project pass
		if (Pass == 1 && FPaths::ProjectDir() == FPaths::EngineDir())
		{
			continue;\
		}
		
		Context.Load(*FPaths::Combine(Pass == 0 ? FPaths::EngineDir() : FPaths::ProjectDir(), TEXT("Config/ConfigRedirects.ini")));

		for (const TPair<FString, FConfigSection>& Section : AsConst(RemapFile))
		{
			if (Section.Key == TEXT("SectionNameRemap"))
			{
				for (const TPair<FName, FConfigValue>& Line : Section.Value)
				{
					SectionRemap.Add(Line.Key.ToString(), Line.Value.GetSavedValue());
				}
			}
			else
			{
				TMap<FString, FString>& KeyRemaps = KeyRemap.FindOrAdd(Section.Key);
				for (const TPair<FName, FConfigValue>& Line : Section.Value)
				{
					KeyRemaps.Add(Line.Key.ToString(), Line.Value.GetSavedValue());
				}
			}
		}
	}
	
	GAllowConfigRemapWarning = true;
}

void FConfigCacheIni::InitializeConfigSystem()
{
	// cache existence of a few key files that may be checked over and over
	// this could be done with the Staged caches, but this will at least speed up a repeated FilExists check when not using binaryconfig
	GConfigLayers[0].bHasCheckedExist = true;
	GConfigLayers[0].bExists = DoesConfigFileExistWrapper(*FString(GConfigLayers[0].Path).Replace(TEXT("{ENGINE}/"), *FPaths::EngineDir()));
	GPluginLayers[0].bHasCheckedExist = true;
	GPluginLayers[0].bExists = DoesConfigFileExistWrapper(*FString(GPluginLayers[0].Path).Replace(TEXT("{ENGINE}/"), *FPaths::EngineDir()));

	// assign the G***Ini strings for the known ini's
	#define ASSIGN_GLOBAL_INI_STRING(IniName) G##IniName##Ini = FString(#IniName);
		// GEngineIni = FString("Engine")
		ENUMERATE_KNOWN_INI_FILES(ASSIGN_GLOBAL_INI_STRING);
	#undef ASSIGN_GLOBAL_INI_STRING

	
	InitializeConfigRemap();
	
#if PLATFORM_SUPPORTS_BINARYCONFIG && PRELOAD_BINARY_CONFIG && !WITH_EDITOR && WITH_CLIENT_CODE
	// attempt to load from staged binary config data
	const bool bCommandLineRequestsTextConfig = FParse::Param(FCommandLine::Get(), TEXT("textconfig")) // Explicitly requesting textconfig
#if ALLOW_INI_OVERRIDE_FROM_COMMANDLINE
		|| (FCString::Stristr(FCommandLine::Get(), CommandlineOverrideSpecifiers::IniFileOverrideIdentifier) != nullptr) // Implicit, requesting a file to be loaded as text from disk
#endif
		;
	if (!bCommandLineRequestsTextConfig &&
		FConfigCacheIni::CreateGConfigFromSaved(nullptr))
	{
		FConfigContext Context = FConfigContext::ReadIntoGConfig();
		LoadRemainingConfigFiles(Context);
	}
	else
#endif
	{
		// Bootstrap the Ini config cache
		FString IniBootstrapFilename;
		if (FParse::Value(FCommandLine::Get(), TEXT("IniBootstrap="), IniBootstrapFilename))
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(IniBootstrap);
			TArray<uint8> FileContent;
			if (FFileHelper::LoadFileToArray(FileContent, *IniBootstrapFilename, FILEREAD_Silent))
			{
				FMemoryReader MemoryReader(FileContent, true);
				GConfig = new FConfigCacheIni(EConfigCacheType::Temporary, NAME_None /*Platform*/, true /* bInGloballyRegistered */);
				GConfig->SerializeStateForBootstrap_Impl(MemoryReader);
				GConfig->bIsReadyForUse = true;
				TRACE_CPUPROFILER_EVENT_SCOPE(ConfigReadyForUseBroadcast);
				FCoreDelegates::TSConfigReadyForUse().Broadcast();
				return;
			}
			else
			{
				FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Unable to bootstrap from archive %s, will fallback on normal initialization\n"), *IniBootstrapFilename);
			}
		}

		// Perform any upgrade we need before we load any configuration files
		FConfigManifest::UpgradeFromPreviousVersions();

		// create GConfig
		GConfig = new FConfigCacheIni(EConfigCacheType::DiskBacked, FPlatformProperties::IniPlatformName(), true /* bInGloballyRegistered */);

	#if UE_ENABLE_CONFIG_SYSTEM
		// create a context object that we will use for all of the main ini files
		FConfigContext Context = FConfigContext::ReadIntoGConfig();

		// load in the default ini files
		bool bEngineConfigCreated = InitializeKnownConfigFiles(Context);

		// verify if needed
		const bool bIsGamelessExe = !FApp::HasProjectName();
		if (!bIsGamelessExe)
		{
			// Now check and see if our game is correct if this is a game agnostic binary
			if (GIsGameAgnosticExe && !bEngineConfigCreated)
			{
				const FText AbsolutePath = FText::FromString(IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*FPaths::GetPath(GEngineIni)));
				//@todo this is too early to localize
				const FText Message = FText::Format(NSLOCTEXT("Core", "FirstCmdArgMustBeGameName", "'{0}' must exist and contain a DefaultEngine.ini."), AbsolutePath);
				if (!GIsBuildMachine)
				{
					FMessageDialog::Open(EAppMsgType::Ok, Message);
				}
				FApp::SetProjectName(TEXT("")); // this disables part of the crash reporter to avoid writing log files to a bogus directory
				if (!GIsBuildMachine)
				{
					exit(1);
				}
				UE_LOGF(LogInit, Fatal, "%ls", *Message.ToString());
			}
		}

		// load editor, etc config files
		LoadRemainingConfigFiles(Context);
	#endif // UE_ENABLE_CONFIG_SYSTEM
	}

	FCoreDelegates::TSOnConfigSectionsChanged().AddStatic(OnConfigSectionsChanged);

	// now we can make use of GConfig
	GConfig->bIsReadyForUse = true;

#if WITH_EDITOR
	// this needs to be called after setting bIsReadyForUse, because it uses ProjectDir, and bIsReadyForUse can reset the 
	// ProjectDir array while the async threads are using it and crash
	AsyncInitializeConfigForPlatforms();
#endif

	TRACE_CPUPROFILER_EVENT_SCOPE(ConfigReadyForUseBroadcast);
	FCoreDelegates::TSConfigReadyForUse().Broadcast();
}

const FString& FConfigCacheIni::GetCustomConfigString()
{
	static FString CustomConfigString;
	static bool bInitialized = false;
	if (!bInitialized)
	{
		bInitialized = true;

		// Set to compiled in value, then possibly override
		bool bCustomConfigOverrideApplied = false;
		CustomConfigString = TEXT(CUSTOM_CONFIG);

#if ALLOW_INI_OVERRIDE_FROM_COMMANDLINE
		if (FParse::Value(FCommandLine::Get(), CommandlineOverrideSpecifiers::CustomConfigIdentifier, CustomConfigString))
		{
			bCustomConfigOverrideApplied = true;
			UE_LOGF(LogConfig, Log, "Overriding CustomConfig from %ls to %ls using -customconfig cmd line param", TEXT(CUSTOM_CONFIG), *CustomConfigString);
		}
#endif

#ifdef UE_USE_COMMAND_LINE_PARAM_FOR_CUSTOM_CONFIG
		FString CustomName = UE_STRINGIZE(UE_USE_COMMAND_LINE_PARAM_FOR_CUSTOM_CONFIG);
		if (!bCustomConfigOverrideApplied && FParse::Param(FCommandLine::Get(), *CustomName))
		{
			bCustomConfigOverrideApplied = true;
			CustomConfigString = CustomName;
			UE_LOGF(LogConfig, Log, "Overriding CustomConfig from %ls to %ls using a custom cmd line param", TEXT(CUSTOM_CONFIG), *CustomConfigString);
		}
#endif

		if (!bCustomConfigOverrideApplied && !CustomConfigString.IsEmpty())
		{
			UE_LOGF(LogConfig, Log, "Using compiled CustomConfig %ls", *CustomConfigString);
		}
	}
	return CustomConfigString;
}

bool FConfigCacheIni::LoadGlobalIniFile(FString& OutFinalIniFilename, const TCHAR* BaseIniName, const TCHAR* Platform, bool bForceReload, bool bRequireDefaultIni, bool bAllowGeneratedIniWhenCooked, bool bAllowRemoteConfig, const TCHAR* GeneratedConfigDir, FConfigCacheIni* ConfigSystem)
{
	FConfigContext Context = FConfigContext::ReadIntoConfigSystem(ConfigSystem, Platform);
	if (GeneratedConfigDir != nullptr)
	{
		Context.GeneratedConfigDir = GeneratedConfigDir;
	}
	Context.bForceReload = bForceReload;
	Context.bAllowGeneratedIniWhenCooked = bAllowGeneratedIniWhenCooked;
	Context.bAllowRemoteConfig = bAllowRemoteConfig;
	return Context.Load(BaseIniName, OutFinalIniFilename);
}

bool FConfigCacheIni::LoadLocalIniFile(FConfigFile & ConfigFile, const TCHAR * IniName, bool bIsBaseIniName, const TCHAR * Platform, bool bForceReload)
{
	FConfigContext Context = bIsBaseIniName ? FConfigContext::ReadIntoLocalFile(ConfigFile, Platform) : FConfigContext::ReadSingleIntoLocalFile(ConfigFile, Platform);
	Context.bForceReload = bForceReload;
	return Context.Load(IniName);
}

bool FConfigCacheIni::LoadExternalIniFile(FConfigFile & ConfigFile, const TCHAR * IniName, const TCHAR * EngineConfigDir, const TCHAR * SourceConfigDir, bool bIsBaseIniName, const TCHAR * Platform, bool bForceReload, bool bWriteDestIni, bool bAllowGeneratedIniWhenCooked, const TCHAR * GeneratedConfigDir)
{
	LLM_SCOPE(ELLMTag::ConfigSystem);

	// 	could also set Context.bIsHierarchicalConfig instead of the ?: operator
	FConfigContext Context = bIsBaseIniName ? FConfigContext::ReadIntoLocalFile(ConfigFile, Platform) : FConfigContext::ReadSingleIntoLocalFile(ConfigFile, Platform);
	Context.EngineConfigDir = EngineConfigDir;
	Context.ProjectConfigDir = SourceConfigDir;
	Context.bForceReload = bForceReload;
	Context.bAllowGeneratedIniWhenCooked = bAllowGeneratedIniWhenCooked;
	Context.GeneratedConfigDir = GeneratedConfigDir;
	Context.bWriteDestIni = bWriteDestIni;
#if UE_WITH_CONFIG_TRACKING
	if (ConfigFile.LoadType == UE::ConfigAccessTracking::ELoadType::Uninitialized)
	{
		ConfigFile.LoadType = bIsBaseIniName ? UE::ConfigAccessTracking::ELoadType::ExternalIniFile :
			UE::ConfigAccessTracking::ELoadType::ExternalSingleIniFile;
	}
#endif
	return Context.Load(IniName);
}

FConfigFile* FConfigCacheIni::FindPlatformConfig(const TCHAR* IniName, const TCHAR* Platform)
{
	FConfigLogContextScope LogScope(TEXT("FindPlatformConfig"), nullptr, nullptr);

	if (Platform != nullptr && FCString::Stricmp(Platform, ANSI_TO_TCHAR(FPlatformProperties::IniPlatformName())) != 0)
	{
#if ALLOW_OTHER_PLATFORM_CONFIG
		return FConfigCacheIni::ForPlatform(Platform)->FindConfigFile(IniName);
#else
		return nullptr;
#endif
	}

	if (GConfig != nullptr)
	{
		return GConfig->FindConfigFile(IniName);
	}

	return nullptr;
}

FConfigFile* FConfigCacheIni::FindOrLoadPlatformConfig(FConfigFile& LocalFile, const TCHAR* IniName, const TCHAR* Platform)
{
	FConfigLogContextScope LogScope(TEXT("FindOrLoadPlatformConfig"), nullptr, nullptr);

	FConfigFile* File = FindPlatformConfig(IniName, Platform);
	if (File == nullptr)
	{
		FConfigContext Context = FConfigContext::ReadIntoLocalFile(LocalFile, Platform);
		Context.Load(IniName);
		File = &LocalFile;
	}

	return File;
}

void FConfigCacheIni::LoadConsoleVariablesFromINI()
{
#if !DISABLE_CHEAT_CVARS
	{
		const TCHAR* StartupSectionName = TEXT("Startup");
		FString PlatformName = FPlatformProperties::IniPlatformName();
		FString StartupPlatformSectionName = FString::Printf(TEXT("Startup_%s"), *PlatformName);
		FString ConsoleVariablesPath = FPaths::EngineDir() + TEXT("Config/ConsoleVariables.ini");

		// First we read from "../../../Engine/Config/ConsoleVariables.ini" [Startup] section if it exists
		// This is the only ini file where we allow cheat commands (this is why it's not there for UE_BUILD_SHIPPING || UE_BUILD_TEST)
		UE::ConfigUtilities::ApplyCVarSettingsFromIni(StartupSectionName, *ConsoleVariablesPath, ECVF_SetByConsoleVariablesIni, true);
		UE::ConfigUtilities::ApplyCVarSettingsFromIni(*StartupPlatformSectionName, *ConsoleVariablesPath, ECVF_SetByConsoleVariablesIni, true);

		#if !UE_BUILD_SHIPPING
		{
			FString OverrideConsoleVariablesPath;
			FParse::Value(FCommandLine::Get(), TEXT("-cvarsini="), OverrideConsoleVariablesPath);

			if (!OverrideConsoleVariablesPath.IsEmpty())
			{
				ensureMsgf(FPaths::FileExists(OverrideConsoleVariablesPath), TEXT("-cvarsini's file %s doesn't exist"), *OverrideConsoleVariablesPath);
				UE::ConfigUtilities::ApplyCVarSettingsFromIni(StartupSectionName, *OverrideConsoleVariablesPath, ECVF_SetByConsoleVariablesIni, true);
				UE::ConfigUtilities::ApplyCVarSettingsFromIni(*StartupPlatformSectionName, *OverrideConsoleVariablesPath, ECVF_SetByConsoleVariablesIni, true);
			}

		}
		#endif
	}
#endif // !DISABLE_CHEAT_CVARS

	// We also apply from Engine.ini [ConsoleVariables] section
	UE::ConfigUtilities::ApplyCVarSettingsFromIni(TEXT("ConsoleVariables"), *GEngineIni, ECVF_SetBySystemSettingsIni);

#if WITH_EDITOR
	// We also apply from DefaultEditor.ini [ConsoleVariables] section
	UE::ConfigUtilities::ApplyCVarSettingsFromIni(TEXT("ConsoleVariables"), *GEditorIni, ECVF_SetBySystemSettingsIni);
#endif	//WITH_EDITOR

	IConsoleManager::Get().CallAllConsoleVariableSinks();
}

FString FConfigCacheIni::NormalizeConfigIniPath(const FString& NonNormalizedPath)
{
	// CreateStandardFilename may not actually do anything in certain cases (e.g., if we detect a network drive, non-root drive, etc.)
	// At a minimum, we will remove double slashes to try and fix some errors.
	return FPaths::CreateStandardFilename(FPaths::RemoveDuplicateSlashes(NonNormalizedPath));
}

FArchive& operator<<(FArchive& Ar, FConfigFile& ConfigFile)
{
	bool bDirty = ConfigFile.Dirty;
	bool bNoSave = ConfigFile.NoSave;
	bool bHasPlatformName = ConfigFile.bHasPlatformName;

	Ar << static_cast<FConfigFile::Super&>(ConfigFile);
	Ar << bDirty;
	Ar << bNoSave;
	Ar << bHasPlatformName;

	Ar << ConfigFile.Name;
	Ar << ConfigFile.PlatformName;
	Ar << ConfigFile.PerObjectConfigArrayOfStructKeys;

	if (Ar.IsLoading())
	{
		ConfigFile.Dirty = bDirty;
		ConfigFile.NoSave = bNoSave;
		ConfigFile.bHasPlatformName = bHasPlatformName;
#if UE_WITH_CONFIG_TRACKING
		ConfigFile.LoadType = UE::ConfigAccessTracking::ELoadType::Manual;
#endif
	}

	return Ar;
}

void FConfigFile::UpdateSections(const TCHAR* DiskFilename, const TCHAR* IniRootName/*=nullptr*/, const TCHAR* OverridePlatform/*=nullptr*/)
{
	// Since we don't want any modifications to other sections, we manually process the file, not read into sections, etc
	// Keep track of existing SectionTexts and orders so that we can preserve the order of the sections in Write to reduce the diff we make to the file on disk
	FString DiskFile;
	FString NewFile;
	TStringBuilder<128> SectionText;
	TMap<FString, FString> SectionTexts;
	TArray<FString> SectionOrder;
	FString SectionName;

	auto AddSectionText = [this, &SectionTexts, &SectionOrder, &SectionName, &SectionText]()
	{
		if (SectionText.Len() == 0)
		{
			// No text in the section, not even a section header, e.g. because this is the prefix section and there was no prefix.
			// Skip the section - add it to SectionTexts or SectionOrder
		}
		else
		{
			if (Contains(SectionName))
			{
				// Do not add the section to SectionTexts, so that Write will skip writing it at all if it is empty
				// But do add it to SectionOrder, so that Write will write it to the right location if it is non-empty
			}
			else
			{
				// Check for duplicate sections in the file-on-disk; handle these by combining them.  This will modify the file, but will guarantee that we don't lose data
				FString* ExistingSectionText = SectionTexts.Find(SectionName);
				if (ExistingSectionText)
				{
					ExistingSectionText->Append(SectionText);
				}
				else
				{
					SectionTexts.Emplace(SectionName, SectionText);
				}
			}

			SectionOrder.Add(SectionName);
		}

		// Clear the name and text for the next section
		SectionName.Reset();
		SectionText.Reset();
	};

	SectionName = FString(); // The lines we read before we encounter a section header should be preserved as prefix lines; we implement this by storing them in an empty SectionName
	if (LoadConfigFileWrapper(DiskFilename, DiskFile))
	{
		// walk each line
		const TCHAR* Ptr = DiskFile.Len() > 0 ? *DiskFile : nullptr;
		bool bDone = Ptr ? false : true;
		bool bIsSkippingSection = true;
		while (!bDone)
		{
			// read the next line
			FString TheLine;
			if (FParse::Line(&Ptr, TheLine, true) == false)
			{
				bDone = true;
			}
			else
			{
				// Strip any trailing whitespace to match config parsing.
				TheLine.TrimEndInline();

				// is this line a section? (must be at least [x])
				if (TheLine.Len() > 3 && TheLine[0] == '[' && TheLine[TheLine.Len() - 1] == ']')
				{
					// Add the old section we just finished reading
					AddSectionText();

					// Set SectionName to the name of new section we are about to read
					SectionName = TheLine.Mid(1, TheLine.Len() - 2);
				}

				SectionText.Append(TheLine);
				SectionText.Append(LINE_TERMINATOR);
			}
		}
	}

	// Add the last section we read
	AddSectionText();

	// load the hierarchy up to right before this file
	if (IniRootName != nullptr)
	{
		// we need to make a temporary file, instead of reading directly into FinalCombinedLayers, because
		// the ConfigContext would end up clearing out the contents of the File in GenerateDestIniFile
		// most of this code is temporary workarounds for a better way to update a section in a hierarchical
		// layer (static or dynamic)
		// this would be much simpler if we passed a "Defaults" FConfigFile to WriteInternal, to not use the Branch
		// in this file - the way this function is used, this->Branch is a temp/dummy branch that isn't great for reading
		// against
		
		FConfigFile Combined;

		// read up to right before this file to diff against
		FConfigContext BaseContext = FConfigContext::ReadUpToBeforeFile(Combined, OverridePlatform, DiskFilename);
		BaseContext.Load(IniRootName);
		
		// now when WriteInternal it called below, it will diff against this FinalCombinedLayers
		Branch->FinalCombinedLayers = Combined;
		Branch->Hierarchy = BaseContext.Branch->Hierarchy;
		// this a quick fix to have WriteInternal treat this as a defaults write
		// do we know if it always is default style write here? seems like it from Obj.cpp and LocalizationTargetDetailCustomization.cpp
		// maybe we should call WriteToStringInternal and pass in bIsADefaultsWrite
		Branch->IniPath = TEXT("");
	}

	// pass in DiskFile, so it will not touch the disk if nothing changes
	WriteInternal(DiskFilename, true, SectionTexts, SectionOrder, &DiskFile);
}




bool FConfigFile::UpdateSinglePropertyInSection(const TCHAR* DiskFilename, const TCHAR* PropertyName, const TCHAR* SectionName)
{
	TOptional<FString> PropertyValue;
	if (const FConfigSection* LocalSection = this->FindSection(SectionName))
	{
		if (const FConfigValue* ConfigValue = LocalSection->Find(PropertyName))
		{
			// Use GetSavedValueForWriting rather than GetSavedValue to avoid having this save operation mark the value as having been accessed for dependency tracking
			PropertyValue = ConfigValue->GetSavedValueForWriting();
		}
	}

	FSinglePropertyConfigHelper SinglePropertyConfigHelper(DiskFilename, SectionName, PropertyName, PropertyValue);
	return SinglePropertyConfigHelper.UpdateConfigFile();
}


#if ALLOW_OTHER_PLATFORM_CONFIG
// these are knowingly leaked
TMap<FName, FConfigCacheIni*> FConfigCacheIni::ConfigForPlatform;
FCriticalSection FConfigCacheIni::ConfigForPlatformLock;
#endif

TMap<FName, FConfigCacheIni::FPluginInfo*> FConfigCacheIni::RegisteredPlugins;
FTransactionallySafeCriticalSection FConfigCacheIni::RegisteredPluginsLock;

FTransactionallySafeRWLock FConfigFile::ConfigFileMapLock;

void FConfigCacheIni::AddPluginToAllBranches(FName PluginName, FConfigModificationTracker* ModificationTracker)
{
	AddMultiplePluginsToAllBranches({PluginName}, ModificationTracker);
}

void FConfigCacheIni::RemoveTagFromAllBranches(FName Tag, FConfigModificationTracker* ModificationTracker)
{
	RemoveMultipleTagsFromAllBranches({Tag}, ModificationTracker);
}

void FConfigCacheIni::AddMultiplePluginsToAllBranches(const TArray<FName>& PluginNames, FConfigModificationTracker* ModificationTracker)
{
	GConfig->AddPluginsToBranches(PluginNames, ModificationTracker);
	
#if ALLOW_OTHER_PLATFORM_CONFIG
	FScopeLock Lock(&ConfigForPlatformLock);
	// need to walk over the other platforms without calling ForPlatform because that could end up loading pending plugins
	for (auto Pair : ConfigForPlatform)
	{
		Pair.Value->PendingModificationPlugins.Append(PluginNames);
	}
#endif
}

void FConfigCacheIni::RemoveMultipleTagsFromAllBranches(const TArray<FName>& Tags, FConfigModificationTracker* ModificationTracker)
{
	GConfig->RemoveTagsFromBranches(Tags, ModificationTracker);
	
#if ALLOW_OTHER_PLATFORM_CONFIG
	// need to walk over the other platforms without calling ForPlatform because that could end up loading pending plugins
	for (auto Pair : ConfigForPlatform)
	{
		Pair.Value->RemoveTagsFromBranches(Tags, ModificationTracker);
	}
#endif
}

void FConfigCacheIni::AddPluginsToBranches(const TArray<FName>& PluginNames, FConfigModificationTracker* ModificationTracker)
{
	LLM_SCOPE(ELLMTag::ConfigSystem);
	// @todo make sure we are still pending
	
	TMap<FConfigBranch*, TArray<FDynamicLayerInfo>> AllDynamicLayers;
	
	for (FName PluginName : PluginNames)
	{
		
		FPluginInfo* PluginInfo = nullptr;
		{
			UE::TScopeLock Lock(RegisteredPluginsLock);
			
			PluginInfo = RegisteredPlugins.FindRef(PluginName);
			if (PluginInfo == nullptr)
			{
				UE_LOGF(LogConfig, Warning, "Attempting to load a dynamic plugin (%ls) that was not registered ahead of time!", *PluginName.ToString());
				return;
			}
		}
		
		FString PluginConfigDir = FPaths::Combine(PluginInfo->PluginDir, TEXT("Config"));
		FString PlatformNameStr(PlatformName.ToString());
		TSet<FString> SlowPluginConfigs;
		// if we already cached this plugin's configs offline, use it
		TSet<FString>* PluginConfigs = StagedPluginConfigCache.Find(PluginName);
		bool bNamesAreFullPaths = true;
		if (PluginConfigs == nullptr)
		{
			bNamesAreFullPaths = false;
			PluginConfigs = &SlowPluginConfigs;
	
			FString PlatformConfigDir = FPaths::Combine(PluginConfigDir, PlatformNameStr);
			TArray<FString> LocalConfigs;
			IFileManager::Get().FindFiles(LocalConfigs, *PluginConfigDir, TEXT("ini"));
			IFileManager::Get().FindFiles(LocalConfigs, *PlatformConfigDir, TEXT("ini"));
	
			// if this plugin has any platform extensions, then we need to look in them for files, in so that we can load them
			// even if there is no platform-less config file in the plugin itself
			for (FString& ChildPluginDir : PluginInfo->ChildPluginDirs)
			{
				if (ChildPluginDir.Contains(*FString::Printf(TEXT("/%s/"), *PlatformNameStr)))
				{
					FString PlatformExtConfigDir = FPaths::Combine(ChildPluginDir, TEXT("Config"));
					IFileManager::Get().FindFiles(LocalConfigs, *PlatformExtConfigDir, TEXT("ini"));
				}
			}
	
			SlowPluginConfigs = TSet<FString>(LocalConfigs);
		}

		// make a single context that can be used for all the branches modified by this plugin
		FConfigContext Context = FConfigContext::ReadIntoConfigSystem(this, PlatformNameStr);
		Context.bIsForPluginModification = true;
		Context.PluginModificationPriority = PluginInfo->Priority;
		Context.bIncludeTagNameInBranchName = PluginInfo->bIncludePluginNameInBranchName;
		Context.ChangeTracker = ModificationTracker;
		
		// find branches that are found in the plugin dir or it's platform dirs
		FString StrippedPart = PluginInfo->bIncludePluginNameInBranchName ? PluginName.ToString() : FString();
		FName CurrentPlatform(FPlatformProperties::IniPlatformName());
		
		TSet<FName> LoadedBranches;
		for (const FString& ConfigFilename : *PluginConfigs)
		{
			// make sure this file isn't in a subdir (like Tags), this will make the StagedPluginConfigCache case match the SlowPluginConfigs case
			FString DirName = FPaths::GetCleanFilename(FPaths::GetPath(ConfigFilename));
			if (!DirName.IsEmpty() && DirName != TEXT("Config") && DirName != PlatformNameStr)
			{
				UE_LOGF(LogConfig, Verbose, "Skipping config file %ls in plugin dir %ls, since it's in a subdirectory.", *ConfigFilename, *PluginInfo->PluginDir);
				continue;
			}
			UE_LOGF(LogConfig, Verbose, "Found config file %ls in plugin dir %ls", *ConfigFilename, *PluginInfo->PluginDir);

			FName BranchName = *FPaths::GetBaseFilename(ConfigFilename).Replace(*StrippedPart, TEXT("")).Replace(*PlatformNameStr, TEXT(""));
			if (LoadedBranches.Contains(BranchName))
			{
				continue;
			}
			LoadedBranches.Add(BranchName);
			
			// if we have been tracking loaded files, and we've already loaded this file, we can skip it (it would be the DefaultMyPlugin.ini type of file)
			if (ModificationTracker && ModificationTracker->bSkipLoadingAlreadyLoadedFiles)
			{
				FString FullPath = bNamesAreFullPaths ? ConfigFilename : FPaths::Combine(PluginConfigDir, ConfigFilename);
				if (ModificationTracker->LoadedFiles.Contains(FullPath))
				{
					UE_LOGF(LogConfig, Verbose, "Skipping already loaded file %ls", *FullPath);
					continue;
				}
			}
			
			// look up the branch to see if we can modify it
			FConfigBranch* Branch = FindBranch(BranchName, FString());
			if (Branch == nullptr)
			{
				// don't log out for other platforms, because they are being loadd later and the ModificationTracker doesn't have full context
				// @todo: removed this because it was causing the FilterPlugin.ini files to be logged _a lot_.
				UE_CLOGF(PlatformName == CurrentPlatform, LogConfig, Verbose, "Found unknown .ini file %ls in plugindir %ls", *ConfigFilename, *PluginInfo->PluginDir);
				continue;
			}
			
			UE_LOGF(LogConfig, Verbose, "Modifying branch %ls with plugin ini %ls", *BranchName.ToString(), *ConfigFilename);
			
			Context.ConfigFileTag = PluginName;
			
			// by setting this, Load will not load the layers, it will just call this function 
			Context.HandleLayersFunction = [&AllDynamicLayers, Branch](const TArray<FDynamicLayerInfo>& Layers)
			{
				AllDynamicLayers.FindOrAdd(Branch).Append(Layers);
			};
			Context.Load(*BranchName.ToString());
		}
	}

	for (TPair<FConfigBranch*, TArray<FDynamicLayerInfo>> Pair : AllDynamicLayers)
	{
		Pair.Key->AddDynamicLayersToHierarchy(Pair.Value, ModificationTracker);
	}
}

void FConfigCacheIni::RemoveTagsFromBranches(const TArray<FName>& Tags, FConfigModificationTracker* ModificationTracker)
{
	for (uint8 KnownIndex = 0; KnownIndex < (uint8)EKnownIniFile::NumKnownFiles; KnownIndex++)
	{
		KnownFiles.Branches[KnownIndex].RemoveTagsFromHierarchy(Tags, ModificationTracker);
	}
	for (auto& Pair : OtherFiles)
	{
		Pair.Value->RemoveTagsFromHierarchy(Tags, ModificationTracker);
	}
}

const TSet<FString>* FConfigCacheIni::GetStagedPluginConfigCache(FName PluginName) const
{
	return StagedPluginConfigCache.Find(PluginName);
}

const TSet<FString>* FConfigCacheIni::GetStagedGlobalConfigCache() const
{
	return StagedGlobalConfigCache;
}

#if ALLOW_OTHER_PLATFORM_CONFIG

#if WITH_EDITOR
void FConfigCacheIni::AsyncInitializeConfigForPlatforms()
{
	// make sure any (non-const static) paths the worker threads will use are already initialized
	FPaths::ProjectDir();
	FPlatformMisc::GeneratedConfigDir(); // also inits FPaths::ProjectSavedDir
	FConfigContext::EnsureRequiredGlobalPathsHaveBeenInitialized();
	FPlatformProcess::ApplicationSettingsDir();

	// pre-create all platforms so that the loop below doesn't reallocate anything in the map
	const TMap<FName, FDataDrivenPlatformInfo>& AllPlatformInfos = FDataDrivenPlatformInfoRegistry::GetAllPlatformInfos();
	for (const TPair<FName, FDataDrivenPlatformInfo>& Pair : AllPlatformInfos)
	{
		GetPlatformConfigFutures().Emplace(Pair.Key);
		ConfigForPlatform.Add(Pair.Key, new FConfigCacheIni(EConfigCacheType::Temporary, Pair.Key, true /* bInGloballyRegistered */));
	}

	for (const TPair<FName, FDataDrivenPlatformInfo>& Pair : AllPlatformInfos)
	{
		FName PlatformName = Pair.Key;
		GetPlatformConfigFutures()[PlatformName] = Async(EAsyncExecution::ThreadPool, [PlatformName]
		{
			double Start = FPlatformTime::Seconds();

			FConfigCacheIni* NewConfig = ConfigForPlatform.FindChecked(PlatformName);
			FConfigContext Context = FConfigContext::ReadIntoConfigSystem(NewConfig, PlatformName.ToString());
			InitializeKnownConfigFiles(Context);
	
			UE_LOGF(LogConfig, Display, "Loading %ls ini files took %.2f seconds", *PlatformName.ToString(), FPlatformTime::Seconds() - Start);
		});
	}
}

#endif
#endif

void FConfigCacheIni::PreInitializePlatformPlugins()
{
#if ALLOW_OTHER_PLATFORM_CONFIG
#if WITH_EDITOR

	const TMap<FName, FDataDrivenPlatformInfo>& AllPlatformInfos = FDataDrivenPlatformInfoRegistry::GetAllPlatformInfos();

	TArray<FName> PlatformNames;
	TArray<FDataDrivenPlatformInfo> PlatformInfos;
	AllPlatformInfos.GetKeys(PlatformNames);

	for (FName PlatformName : PlatformNames)
	{
		if (PlatformName != NAME_None)
		{
			// GetPlatformConfigFutures will be called inside FConfigCacheIni::ForPlatform. 
			// To avoid deadlocks wait for all futures here before entering the ParallelFor.
			if (auto* PlatformCache = GetPlatformConfigFutures().Find(PlatformName))
			{
				PlatformCache->Get();
			}
		}
	}

	ParallelFor(PlatformNames.Num(), [PlatformNames, AllPlatformInfos](int32 Index)
	{
		FName PlatfName = PlatformNames[Index];

		// if we have the editor and we are using -game
		// only need to instantiate the current platform 
		if (IsRunningGame())
		{
			if (PlatfName != FPlatformProperties::IniPlatformName())
			{
				return;
			}
		}

		const FDataDrivenPlatformInfo* Info = AllPlatformInfos.Find(PlatfName);

		if (Info->bEnabledForUse)
		{
			// Calling ForPlatform will invoke AddPluginToBranches on any pending 
			// plugins found for this platform's config. We do this beforehand on
			// worker threads to speed up platform initialization.
			FConfigCacheIni::ForPlatform(PlatfName);
		}
	}, /*bForceSingleThread*/ false);

#endif
#endif
}

FConfigCacheIni* FConfigCacheIni::ForPlatform(FName PlatformName)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FConfigCacheIni::ForPlatform);

#if ALLOW_OTHER_PLATFORM_CONFIG
	check(GConfig != nullptr && GConfig->bIsReadyForUse);

	// use GConfig when no platform is specified
	if (PlatformName == NAME_None)
	{
		return GConfig;
	}

#if WITH_EDITOR
	// they are likely already loaded, but just block to make sure
	{
		if (auto* PlatformCache = GetPlatformConfigFutures().Find(PlatformName))
		{
			PlatformCache->Get();
		}
		else
		{
			return GConfig;
		}
	}
#endif

	FConfigCacheIni* PlatformConfig = nullptr;
	TArray<FName> PendingModificationPlugins;

	// protect against other threads clearing the array, or two threads trying to read in a missing platform at the same time
	FScopeLock Lock(&ConfigForPlatformLock);

	PlatformConfig = ConfigForPlatform.FindRef(PlatformName);

	// read any missing platform configs now, on demand (this will happen when WITH_EDITOR is 0)
	if (PlatformConfig == nullptr)
	{
		double Start = FPlatformTime::Seconds();

		PlatformConfig = ConfigForPlatform.Add(PlatformName, new FConfigCacheIni(EConfigCacheType::Temporary, PlatformName, true /* bInGloballyRegistered */));
		FConfigContext Context = FConfigContext::ReadIntoConfigSystem(PlatformConfig, PlatformName.ToString());
		InitializeKnownConfigFiles(Context);

		UE_LOGF(LogConfig, Display, "Read in platform %ls ini files took %.2f seconds", *PlatformName.ToString(), FPlatformTime::Seconds() - Start);
	}

	PendingModificationPlugins = MoveTemp(PlatformConfig->PendingModificationPlugins);
	PlatformConfig->PendingModificationPlugins.Empty();

	// delayed plugin injection
	PlatformConfig->AddPluginsToBranches(PendingModificationPlugins, nullptr);

	return PlatformConfig;

#else
	UE_LOGF(LogConfig, Error, "FConfigCacheIni::ForPlatform cannot be called when not in a developer tool");
	return nullptr;
#endif
}

void FConfigCacheIni::ClearOtherPlatformConfigs()
{
#if ALLOW_OTHER_PLATFORM_CONFIG
	// this will read in on next call to ForPlatform()
	FScopeLock Lock(&ConfigForPlatformLock);
	ConfigForPlatform.Empty();
#endif
}

void FConfigCacheIni::RegisterPlugin(FName PluginName, const FString& PluginDir, const TArray<FString>& ChildPluginDirs, DynamicLayerPriority Priority, bool bIncludePluginNameInBranchName)
{
	FPluginInfo* Info = new FPluginInfo();

	Info->PluginDir = PluginDir;
	Info->ChildPluginDirs = ChildPluginDirs;
	Info->Priority = Priority;
	Info->bIncludePluginNameInBranchName = bIncludePluginNameInBranchName;

	UE::TScopeLock Lock(RegisteredPluginsLock);
	RegisteredPlugins.Add(PluginName, Info);
}



double GPrepareForLoadTime = 0;
double GPerformLoadTime = 0;
double GConfigShrinkTime = 0;

#if !UE_BUILD_SHIPPING
class FIniExec : public FSelfRegisteringExec
{
	virtual bool Exec_Dev(class UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override
	{
		if (!FParse::Command(&Cmd, TEXT("CONFIG")))
		{
			return false;
		}

		if (FParse::Command(&Cmd, TEXT("AddDyn")))
		{
			TCHAR BranchName[256];
			TCHAR Filename[1024];
			FConfigModificationTracker ChangeTracker;

			if (FParse::Token(Cmd, BranchName, UE_ARRAY_COUNT(BranchName), true) &&
				FParse::Token(Cmd, Filename, UE_ARRAY_COUNT(Filename), true))
			{
				FConfigBranch* Branch = GConfig->FindBranch(NAME_None, BranchName);
				
				if (Branch != nullptr)
				{
					Branch->AddDynamicLayerToHierarchy(Filename, &ChangeTracker);
					Ar.Logf(TEXT("Modified sections:"));
					for (auto Pair : ChangeTracker.ModifiedSectionsPerBranch)
					{
						Ar.Logf(TEXT("  %s"), *Pair.Key.ToString());
						for (const FString& Section : Pair.Value)
						{
							Ar.Logf(TEXT("    %s"), *Section);
						}
					}
				}
			}
			return true;
		}

		if (FParse::Command(&Cmd, TEXT("RemoveDyn")))
		{
			TCHAR BranchName[256];
			TCHAR Filename[1024];
			FConfigModificationTracker ChangeTracker;

			if (FParse::Token(Cmd, BranchName, UE_ARRAY_COUNT(BranchName), true) &&
				FParse::Token(Cmd, Filename, UE_ARRAY_COUNT(Filename), true))
			{
				FConfigBranch* Branch = GConfig->FindBranch(NAME_None, BranchName);
				
				if (Branch != nullptr)
				{
					Branch->RemoveDynamicLayerFromHierarchy(Filename, &ChangeTracker);
					Ar.Logf(TEXT("Modified sections:"));
					for (auto Pair : ChangeTracker.ModifiedSectionsPerBranch)
					{
						Ar.Logf(TEXT("  %s"), *Pair.Key.ToString());
						for (const FString& Section : Pair.Value)
						{
							Ar.Logf(TEXT("    %s"), *Section);
						}
					}
				}
			}

			return true;
		}

		if (FParse::Command(&Cmd, TEXT("AddTrackedDyn")))
		{
			TCHAR BranchName[256];
			TCHAR Filename[1024];

			if (FParse::Token(Cmd, BranchName, UE_ARRAY_COUNT(BranchName), true) &&
				FParse::Token(Cmd, Filename, UE_ARRAY_COUNT(Filename), true))
			{
				FConfigBranch* Branch = GConfig->FindBranch(NAME_None, BranchName);
				UE::DynamicConfig::PerformDynamicConfig("TrackedDyn", [Branch, Filename](FConfigModificationTracker* ChangeTracker)
					{
						// set which sections to track for cvars, with their priority
						ChangeTracker->CVars.Add(TEXT("PluginCVars_LowPriority")).CVarPriority = (int)ECVF_SetByPluginLowPriority;
						ChangeTracker->CVars.Add(TEXT("PluginCVars_HighPriority")).CVarPriority = (int)ECVF_SetByPluginHighPriority;

						FDynamicLayerInfo Info { Filename, "TrackedDyn", (uint16)DynamicLayerPriority::Unknown };
						Branch->AddDynamicLayersToHierarchy({ Info }, ChangeTracker);
					});
			}
		}

		if (FParse::Command(&Cmd, TEXT("RemoveTrackedDyn")))
		{
			UE::DynamicConfig::PerformDynamicConfig("TrackedDyn", [](FConfigModificationTracker* ChangeTracker)
				{
					FConfigCacheIni::RemoveTagFromAllBranches("TrackedDyn", ChangeTracker);
					IConsoleManager::Get().UnsetAllConsoleVariablesWithTag("TrackedDyn");
				});
		}

		if (FParse::Command(&Cmd, TEXT("Diff")))
		{
			TCHAR BranchName[256];
			if (FParse::Token(Cmd, BranchName, UE_ARRAY_COUNT(BranchName), true))
			{
				FConfigBranch* Branch = GConfig->FindBranch(BranchName, BranchName);

				if (Branch != nullptr)
				{
					FConfigCommandStream Diff = CalculateDiff(Branch->FinalCombinedLayers, Branch->InMemoryFile);
					FString Output;
					BuildOutputString(Output, Diff);
					Ar.Logf(TEXT("Disk -> InMemory Diff of %s:\n%s"), BranchName, *Output);
				}
			}
			return true;
		}
		
		if (FParse::Command(&Cmd, TEXT("Flush")))
		{
			TCHAR BranchName[256];
			if (FParse::Token(Cmd, BranchName, UE_ARRAY_COUNT(BranchName), true))
			{
				GConfig->Flush(false, BranchName);
			}
			else
			{
				GConfig->Flush(false);
			}
		}

		if (FParse::Command(&Cmd, TEXT("Unload")))
		{
			TCHAR BranchName[256];
			if (FParse::Token(Cmd, BranchName, UE_ARRAY_COUNT(BranchName), true))
			{
				GConfig->SafeUnloadBranch(BranchName);
			}
		}

		if (FParse::Command(&Cmd, TEXT("UnloadAll")))
		{
			for (const FString& Filename : GConfig->GetFilenames())
			{
				GConfig->SafeUnloadBranch(*Filename);
			}
		}

		if (FParse::Command(&Cmd, TEXT("AddHotFix")))
		{
			TCHAR FileName[256];
			if (FParse::Token(Cmd, FileName, UE_ARRAY_COUNT(FileName), true))
			{
				FString FilenameBase = FPaths::GetBaseFilename(FileName);
				FConfigBranch* Branch = GConfig->FindBranch(*FilenameBase, FilenameBase);
				
				if (Branch != nullptr)
				{
					FString Contents;
					if (FFileHelper::LoadFileToString(Contents, FileName))
					{
						UE::DynamicConfig::PerformDynamicConfig("HotfixTest", [Branch, &FileName, &Contents](FConfigModificationTracker* ChangeTracker)
							{
								Branch->AddDynamicLayerStringToHierarchy(FileName, Contents, "HotfixTest", DynamicLayerPriority::Hotfix, ChangeTracker);
							});
					}
				}
			}
		}
		
		if (FParse::Command(&Cmd, TEXT("RemoveHotFixes")))
		{
			UE::DynamicConfig::PerformDynamicConfig("HotfixTest", [](FConfigModificationTracker* ChangeTracker)
			{
				FConfigCacheIni::RemoveTagFromAllBranches("HotfixTest", ChangeTracker);
			});
		}
		
		if (FParse::Command(&Cmd, TEXT("RemoveSection")))
		{
			TCHAR BranchName[256];
			TCHAR Section[256];

			if (FParse::Token(Cmd, BranchName, UE_ARRAY_COUNT(BranchName), true) &&
				FParse::Token(Cmd, Section, UE_ARRAY_COUNT(Section), true))
			{
				bool bRemovedSomething = GConfig->RemoveSectionFromBranch(Section, BranchName);
				if (bRemovedSomething)
				{
					Ar.Logf(TEXT("Successfully removed '%s' from layer(s) in %s"), Section, BranchName);						
				}
				else
				{
					Ar.Logf(TEXT("Nothing was removed from %s (either branch wasn't found or the section '%s' wasn't)"), BranchName, Section);
				}
			}
			else
			{
				Ar.Logf(TEXT("Usage: config RemoveSection <BranchName> <Section>"));	
			}
		}
		
		if (FParse::Command(&Cmd, TEXT("Timing")))
		{
			Ar.Logf(TEXT("INITIME : PrepareForLoad: %fms, PreformLoad: %fms, Shrink: %fms"), GPrepareForLoadTime * 1000.0, GPerformLoadTime * 1000.0, GConfigShrinkTime * 1000.0);
		}

		if (FParse::Command(&Cmd, TEXT("Shrink")))
		{
			TCHAR BranchName[256];
			if (FParse::Token(Cmd, BranchName, UE_ARRAY_COUNT(BranchName), true))
			{
				FConfigBranch* Branch = GConfig->FindBranch(BranchName, BranchName);
				if (Branch)
				{
					Branch->Shrink();
				}
			}
		}
		
		if (FParse::Command(&Cmd, TEXT("MemUsage")))
		{
			// parse options (default is simple, print to log, 10kb cutoff)
			bool bUseDetailed = FParse::Param(Cmd, TEXT("detailed"));
			FString CSVFilename;
			bool bWriteToCSV = FParse::Value(Cmd, TEXT("-csv="), CSVFilename);
			bWriteToCSV = bWriteToCSV || FParse::Param(Cmd, TEXT("csv"));
			int CutoffKB = 10;
			FParse::Value(Cmd, TEXT("Cutoff="), CutoffKB);

			// handle CSV output
			FArchive* CSV = nullptr;
			if (bWriteToCSV)
			{
				if (CSVFilename.IsEmpty())
				{
					CSVFilename = FPaths::Combine(FPaths::ProjectLogDir(), TEXT("ConfigMemUsage.csv"));
				}
				CSV = IFileManager::Get().CreateFileWriter(*CSVFilename, FILEWRITE_AllowRead);
				if (CSV == nullptr)
				{
					Ar.Logf(TEXT("Unable to create CSV file for writing: '%s'"), *CSVFilename);
					return true;
				}

				Ar.Logf(TEXT("Dumping to CSV file: '%s'"), *CSVFilename);
			}

			// init counters
			uint64 Total = 0;
			int NumSkipped = 0;
			uint64 SkippedTotal = 0;
			int Unloaded = 0;
			uint64 UnloadedTotal = 0;
			int SingleSection = 0;
			uint64 SingleSectionTotal = 0;
			int NoSection = 0;
			uint64 NoSectionTotal = 0;

			uint64 SlackTotal = 0;
			for (const FString& Filename : GConfig->GetFilenames())
			{
				FConfigBranch* Branch = GConfig->FindBranchWithNoReload(*Filename, Filename);

				FDetailedConfigMemUsage MemAr(Branch, bUseDetailed);

				uint64 Mem = MemAr.GetMax();
				Total += Mem;
				SlackTotal += MemAr.GetMax() - MemAr.GetNum();

				if (Branch->bIsSafeUnloaded)
				{
					Unloaded++;
					UnloadedTotal += Mem;
				}
				else if (Branch->InMemoryFile.Num() == 1)
				{
					SingleSection++;
					SingleSectionTotal += Mem;
				}
				else if (Branch->InMemoryFile.Num() == 0)
				{
					NoSection++;
					NoSectionTotal += Mem;
				}

				// don't bother printing the neglibly sized ones as they are just noise, so cut off anything < 10kb
				if (Mem < CutoffKB * 1024)
				{
					NumSkipped++;
					SkippedTotal += Mem;
				}
				else
				{
					FArchiveCountConfigMem SectionMem;
					FArchiveCountConfigMem ValueMem;




					if (bWriteToCSV)
					{
						CSV->Logf(TEXT("%0.2fmb,%0.2fmb,%s"), (double)MemAr.GetNum() / 1024.0 / 1024.0, (double)MemAr.GetMax() / 1024.0 / 1024.0, *Filename);
					}
					else
					{
						Ar.Logf(TEXT("[%0.2fmb / %0.2fmb] - %s"), (double)MemAr.GetNum() / 1024.0 / 1024.0, (double)MemAr.GetMax() / 1024.0 / 1024.0, *Filename);
					}
					bool bPrintedHeader = false;
					for (auto& Pair : MemAr.PerLayerInfo)
					{
						if (Pair.Value.GetMax() >= CutoffKB * 1024)
						{
							if (bWriteToCSV)
							{
								if (!bPrintedHeader)
								{
									CSV->Logf(TEXT(",Large layers:"));
								}
								CSV->Logf(TEXT(",,%0.2fmb,%0.2fmb,%s"), (double)Pair.Value.GetNum() / 1024.0 / 1024.0, (double)Pair.Value.GetMax() / 1024.0 / 1024.0, *Pair.Key);
							}
							else
							{
								if (!bPrintedHeader)
								{
									Ar.Logf(TEXT("  Large layers:"));
								}
								Ar.Logf(TEXT("    [%0.2fmb / %0.2fmb] - %s"), (double)Pair.Value.GetNum() / 1024.0 / 1024.0, (double)Pair.Value.GetMax() / 1024.0 / 1024.0, *Pair.Key);
							}
							bPrintedHeader = true;
						}
					}
					bPrintedHeader = false;
					for (auto& Pair : MemAr.PerSectionInfo)
					{
						if (Pair.Value.GetMax() >= CutoffKB * 1024)
						{
							if (bWriteToCSV)
							{
								if (!bPrintedHeader)
								{
									CSV->Logf(TEXT(",Large sections (across all layers):"));
								}
								CSV->Logf(TEXT(",,%0.2fmb,%0.2fmb,%s"), (double)Pair.Value.GetNum() / 1024.0 / 1024.0, (double)Pair.Value.GetMax() / 1024.0 / 1024.0, *Pair.Key);
							}
							else
							{
								if (!bPrintedHeader)
								{
									Ar.Logf(TEXT("  Large sections (across all layers):"));
								}
								Ar.Logf(TEXT("    [%0.2fmb / %0.2fmb] - %s"), (double)Pair.Value.GetNum() / 1024.0 / 1024.0, (double)Pair.Value.GetMax() / 1024.0 / 1024.0, *Pair.Key);
							}
							bPrintedHeader = true;
						}
					}
					bPrintedHeader = false;
					for (auto& Pair : MemAr.PerSectionValueInfo	)
					{
						if (Pair.Value.GetMax() >= CutoffKB * 1024)
						{
							if (bWriteToCSV)
							{
								if (!bPrintedHeader)
								{
									CSV->Logf(TEXT(",Large sections (by values):"));
								}
								CSV->Logf(TEXT(",,%0.2fmb,%0.2fmb,%s"), (double)Pair.Value.GetNum() / 1024.0 / 1024.0, (double)Pair.Value.GetMax() / 1024.0 / 1024.0, *Pair.Key);
							}
							else
							{
								if (!bPrintedHeader)
								{
									Ar.Logf(TEXT("  Large sections (by values):"));
								}
								Ar.Logf(TEXT("    [%0.2fmb / %0.2fmb] - %s"), (double)Pair.Value.GetNum() / 1024.0 / 1024.0, (double)Pair.Value.GetMax() / 1024.0 / 1024.0, *Pair.Key);
							}
							bPrintedHeader = true;
						}
					}
				}
			}

			if (bWriteToCSV)
			{
				CSV->Logf(TEXT(""));
				CSV->Logf(TEXT("%0.2fmb,%d All Configs"), (double)Total / 1024.0 / 1024.0, GConfig->GetFilenames().Num());
				CSV->Logf(TEXT("%0.2fmb,%d Tiny Configs (not displayed above)"), (double)SkippedTotal / 1024.0 / 1024.0, NumSkipped);
				CSV->Logf(TEXT("%0.2fmb,%d Single Section Configs"), (double)SingleSectionTotal / 1024.0 / 1024.0, SingleSection);
				CSV->Logf(TEXT("%0.2fmb,%d ZeroSection Configs"), (double)NoSectionTotal / 1024.0 / 1024.0, NoSection);
				CSV->Logf(TEXT("%0.2fmb,Total Slack (wasted memory)"), (double)SlackTotal / 1024.0 / 1024.0);
				CSV->Logf(TEXT(""));
				if (!bUseDetailed)
				{
					CSV->Logf(TEXT("To get more detailed information, use \"config memusage -detailed\""));
				}
				if (CutoffKB == 10)
				{
					CSV->Logf(TEXT("To change the cutoff, in KB, for small files/layers/sections, use \"config memusage -cutoff=<value>\""));
				}
#if WITH_EDITOR
				CSV->Logf(TEXT("(Note: Editor builds store more layer state, so the memory usage will be higher than in a client build)"));
#endif
			}
			else
			{
				Ar.Logf(TEXT(""));
			Ar.Logf(TEXT("[%0.2fmb] - %d All Configs"), (double)Total / 1024.0 / 1024.0, GConfig->GetFilenames().Num());
			Ar.Logf(TEXT("[%0.2fmb] - %d SafeUnloaded Configs"), (double)UnloadedTotal / 1024.0 / 1024.0, Unloaded);
			Ar.Logf(TEXT("[%0.2fmb] - %d Tiny Configs (not displayed above)"), (double)SkippedTotal / 1024.0 / 1024.0, NumSkipped);
			Ar.Logf(TEXT("[%0.2fmb] - %d Single Section Configs"), (double)SingleSectionTotal / 1024.0 / 1024.0, SingleSection);
			Ar.Logf(TEXT("[%0.2fmb] - %d ZeroSection Configs"), (double)NoSectionTotal / 1024.0 / 1024.0, NoSection);
				Ar.Logf(TEXT("[%0.2fmb] - Total Slack (wasted memory)"), (double)SlackTotal / 1024.0 / 1024.0);
				Ar.Logf(TEXT(""));
				if (!bUseDetailed)
				{
					Ar.Logf(TEXT("To get more detailed information, use \"config memusage -detailed\""));
				}
				if (CutoffKB == 10)
				{
					Ar.Logf(TEXT("To change the cutoff, in KB, for small files/layers/sections, use \"config memusage -cutoff=<value>\""));
				}
				Ar.Logf(TEXT("To save to .csv, use \"config memusage -csv or -csv=<filepath>\""));
#if WITH_EDITOR
				Ar.Logf(TEXT("(Note: Editor builds store more layer state, so the memory usage will be higher than in a client build)"));
#endif
			}
			delete CSV;
		}

		if (FParse::Command(&Cmd, TEXT("Lint")))
		{
			FString BranchName;

			if (!FParse::Token(Cmd, BranchName, true))
			{
				Ar.Logf(TEXT("Usage: config lint <branch> [platform] [-SameAsPrevious] [-MissingBase] [-ArrayDuplicate]\nIf no - options are specified, all issues will be shown"));
			}
			else
			{
				FString PlatformNameString;
				FName PlatformName = FPlatformProperties::IniPlatformName();
				if (FParse::Token(Cmd, PlatformNameString, true))
				{
					if (!PlatformNameString.StartsWith("-"))
					{
						PlatformName = *PlatformNameString;
					}
				}
				bool bShowSameAsPrevious = FParse::Param(Cmd, TEXT("-SameAsPrevious"));
				bool bShowMissingDefault = FParse::Param(Cmd, TEXT("-MissingBase"));
				bool bShowArrayDuplicates = FParse::Param(Cmd, TEXT("-ArrayDuplicate"));
				// if none specified, do all
				if (!bShowSameAsPrevious && !bShowMissingDefault && !bShowArrayDuplicates)
				{
					bShowSameAsPrevious = bShowMissingDefault = bShowArrayDuplicates = true;
				}


				FConfigCacheIni ConfigSystem(EConfigCacheType::Temporary, PlatformName);
				FConfigContext Context = FConfigContext::ReadIntoConfigSystem(&ConfigSystem, PlatformNameString);
				Context.Load(*BranchName);

				FConfigBranch* Branch = ConfigSystem.FindBranch(*BranchName, BranchName);
				if (!Branch)
				{
					Ar.Logf(TEXT("Unknown branch %s"), *BranchName);
					return true;
				}

				auto LogHeaders = [&Ar]( bool& bShownFileHeader, bool& bShownSectionHeader, const FString& Filename, const FString& SectionName)
					{
						bool bIsFirstSection = false;
						if (!bShownFileHeader)
						{
							Ar.Logf(TEXT("-------------------------------------------------------------"));
							Ar.Logf(TEXT("%s:"), *Filename);
							bShownFileHeader = true;
							bIsFirstSection = true;
						}
						if (!bShownSectionHeader)
						{
							if (!bIsFirstSection)
							{
								Ar.Logf(TEXT("  "));
							}
							Ar.Logf(TEXT("  [%s]"), *SectionName);
							bShownSectionHeader = true;
						}
					};

				FConfigFile BaseLayer;
				FConfigFile PreviousLayer;
				// now walk the hierarchy, diffing things
				bool bTrueBaseFound = false;
				bool bBranchBaseFound = false;
				for (const TPair<int32, FUtf8String>& Pair : Branch->Hierarchy)
				{
					const FString Value = FString(Pair.Value);

					if (!bTrueBaseFound)
					{
						// prime the layers with the first, mainly empty layer
						Branch->MergeStaticLayersUpToAndIncluding(Value, PreviousLayer);
						bTrueBaseFound = true;
						continue;
					}

					FConfigFile CurrentCombinedLayer;
					Branch->MergeStaticLayersUpToAndIncluding(Value, CurrentCombinedLayer);
					if (!bBranchBaseFound)
					{
						BaseLayer = CurrentCombinedLayer;
						PreviousLayer = CurrentCombinedLayer;

						// don't do much here, no need to lint against the true base
						bBranchBaseFound = true;
						continue;
					}

					
					const FConfigCommandStream* CurrentLayer = Branch->GetStaticLayer(Value);
					if (CurrentLayer == nullptr)
					{
						// most layers won't exist
						continue;
					}

					bool bShownFileHeader = false;
					// now walk over everthing, and make sure they don't match previous layer, and that they were in the base layer to be initialized correctly
					for (const TPair<FString, FConfigCommandStreamSection>& SectionPair : *CurrentLayer)
					{
						bool bShownSectionHeader = false;
						TMap<FString, uint8> AddRemoveValues;
						const FConfigSection* PreviousSection = PreviousLayer.FindSection(SectionPair.Key);

						for (const TPair<FName, FConfigValue>& ValuePair : SectionPair.Value)
						{
							if (ValuePair.Value.ValueType == FConfigValue::EValueType::Set)
							{
								const FConfigValue* PreviousValue = PreviousSection ? PreviousSection->Find(ValuePair.Key) : nullptr;
								if (PreviousValue == nullptr)
								{
									if (bShowMissingDefault)
									{
										LogHeaders(bShownFileHeader, bShownSectionHeader, Value, SectionPair.Key);
										Ar.Logf(TEXT("  %s -- No Default in Base"), *ValuePair.Key.ToString());
									}
								}
								else if (PreviousValue->GetSavedValueForWriting() == ValuePair.Value.GetSavedValueForWriting())
								{
									if (bShowSameAsPrevious)
									{
										LogHeaders(bShownFileHeader, bShownSectionHeader, Value, SectionPair.Key);
										Ar.Logf(TEXT("  %s -- Identical To Previous"), *ValuePair.Key.ToString());
									}
								}
							}
							else if (ValuePair.Value.ValueType == FConfigValue::EValueType::ArrayAddUnique)
							{
								uint8& Flags = AddRemoveValues.FindOrAdd(ValuePair.Value.GetSavedValueForWriting());
								Flags |= 1;
							}
							else if (ValuePair.Value.ValueType == FConfigValue::EValueType::Remove)
							{
								uint8& Flags = AddRemoveValues.FindOrAdd(ValuePair.Value.GetSavedValueForWriting());
								Flags |= 2;
							}
						}

						if (bShowArrayDuplicates)
						{
							for (const TPair<FString, uint8>& AddRemovePair : AddRemoveValues)
							{
								if (AddRemovePair.Value == 3)
								{
									LogHeaders(bShownFileHeader, bShownSectionHeader, Value, SectionPair.Key);
									Ar.Logf(TEXT("  %s -- Has Add and Remove"), *AddRemovePair.Key);
								}
							}
						}
					};

					PreviousLayer = CurrentCombinedLayer;
				}
			}
		}

		return true;
	}
	
} GConfigExec;
#endif // !UE_BUILD_SHIPPING

#undef LOCTEXT_NAMESPACE
