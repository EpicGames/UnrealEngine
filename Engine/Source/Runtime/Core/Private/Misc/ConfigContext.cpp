// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/ConfigContext.h"
#include "Misc/App.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/ConfigHierarchy.h"
#include "Misc/CoreMisc.h"
#include "Misc/DataDrivenPlatformInfoRegistry.h"
#include "Misc/MessageDialog.h"
#include "Misc/Paths.h"
#include "Misc/RemoteConfigIni.h"
#include "Misc/ScopeLock.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "ProfilingDebugging/AssetMetadataTrace.h"
#include "HAL/FileManager.h"
#include "HAL/LowLevelMemStats.h"
#include "Logging/MessageLog.h"
#include "Async/Async.h"
#include "Async/TaskGraphInterfaces.h"
#include "Trace/Trace.h"
#include "Trace/Trace.inl"

extern double GPrepareForLoadTime;
extern double GPerformLoadTime;

#define LOCTEXT_NAMESPACE "ConfigSystem"

namespace
{
	FString VersionName = "Version";
	FString PreserveName = "Preserve";
	FString LegacyIniVersionString = "IniVersion";
	FString LegacyEngineString = "Engine.Engine";
	FString CurrentIniVersionString = "CurrentIniVersion";
	const TCHAR* SectionsToSaveString = TEXT("SectionsToSave");
	const TCHAR* SaveAllSectionsKey = TEXT("bCanSaveAllSections");

	// some settings for hierarchy keys
	constexpr int32 KeyFlag_UseGlobalCache = 1 << 0;
	constexpr int32 KeyFlag_UsePluginCache = 1 << 1;
	constexpr int32 KeyFlag_AssumeExists = 1 << 2;
	constexpr int32 KeyFlag_Deprecated_Plugin = 1 << 3;
	constexpr int32 KeyFlag_Deprecated_Other = 1 << 4;
	// if you add a KeyFlag, make sure to increase NumFlagsBits below if needed

	constexpr int32 NumStaticLayerBits = 8; // needs to hold the length of GConfigLayers (or any custom layers, which is highly unlikely to ever be longer than GConfigLayers)
	constexpr int32 NumExpansionBits = 5; // needs to hold the length of GConfigExpansions
	constexpr int32 NumPlatformBits = 5; // needs to hold the length of the longest IniPlatformParent chains (which are generally going always be < 4)
	constexpr int32 NumFlagsBits = 8; // needs to hold KeyFlags above

	constexpr int32 LayerOffset = NumExpansionBits + NumPlatformBits + NumFlagsBits;
	constexpr int32 ExpansionOffset = NumPlatformBits + NumFlagsBits;
	constexpr int32 PlatformOffset = NumFlagsBits;
	constexpr int32 FlagsOffset = 0;

	static_assert(UE_ARRAY_COUNT(GConfigLayers) < (1 << NumStaticLayerBits), "Need more NumStaticLayerBits");
	static_assert(UE_ARRAY_COUNT(GConfigExpansions) < (1 << NumExpansionBits), "Need more NumExpansionBits");
	static_assert(NumStaticLayerBits + NumExpansionBits + NumPlatformBits + NumFlagsBits <= 32, "All bits combined is greater than 32");

	constexpr int32 GetStaticKey(int32 LayerIndex, int32 ExpansionIndex, int32 PlatformIndex, int32 Flags)
	{
		return (LayerIndex << LayerOffset) + (ExpansionIndex << ExpansionOffset) + (PlatformIndex << PlatformOffset) + (Flags << FlagsOffset);
	}
}


FConfigContext::FConfigContext(FConfigCacheIni* InConfigSystem, bool InIsHierarchicalConfig, const FString& InPlatform, FConfigFile* DestConfigFile)
	: ConfigSystem(InConfigSystem)
	, Platform(InPlatform)
	, bIsHierarchicalConfig(InIsHierarchicalConfig)
{
	if (DestConfigFile != nullptr)
	{
		ExistingFile = DestConfigFile;
		bDoNotResetConfigFile = true;
	}

	if (Platform.IsEmpty())
	{
		// read from, for instance Windows
		Platform = FPlatformProperties::IniPlatformName();
		// but save Generated ini files to, say, WindowsEditor
		SavePlatform = FPlatformProperties::PlatformName();
	}
	else if (Platform == FPlatformProperties::IniPlatformName())
	{
		// but save Generated ini files to, say, WindowsEditor
		SavePlatform = FPlatformProperties::PlatformName();
	}
	else
	{
		SavePlatform = Platform;
	}

	// now set to defaults anything not already set
	EngineConfigDir = FPaths::EngineConfigDir();
	ProjectConfigDir = FPaths::SourceConfigDir();

	// set settings that apply when using GConfig
	if (ConfigSystem != nullptr && ConfigSystem == GConfig)
	{
		bWriteDestIni = true;
		bUseHierarchyCache = true;
		bAllowGeneratedIniWhenCooked = true;
		bAllowRemoteConfig = true;
	}
}

FConfigContext::~FConfigContext()
{
	delete TemporaryBranch;
}

void FConfigContext::CachePaths()
{
	// these are needed for single ini files
	if (bIsHierarchicalConfig)
	{
		// are we loading a plugin?
		if (ConfigSystem != nullptr)
		{
			UE::TScopeLock Lock(FConfigCacheIni::RegisteredPluginsLock);
			
			FName PluginName = ConfigFileTag == NAME_None ? FName(*BaseIniName) : ConfigFileTag;
			FConfigCacheIni::FPluginInfo* PluginInfo = ConfigSystem->RegisteredPlugins.FindRef(PluginName);
			if (PluginInfo != nullptr)
			{
				bIsForPlugin = true;
				PluginRootDir = PluginInfo->PluginDir;
				ChildPluginBaseDirs = PluginInfo->ChildPluginDirs;
				StagedPluginConfigCache = ConfigSystem->StagedPluginConfigCache.Find(PluginName);
			}
			StagedGlobalConfigCache = ConfigSystem->StagedGlobalConfigCache;
		}
		
		// for the hierarchy replacements, we need to have a directory called Config - or we will have to do extra processing for these non-standard cases
		check(EngineConfigDir.EndsWith(TEXT("Config/")));
		// allow for an empty projcet config dir, which means (below) to not load any of the {PROJECT} layers
		check(ProjectConfigDir.Len() == 0 || ProjectConfigDir.EndsWith(TEXT("Config/")));

		EngineRootDir = FPaths::GetPath(FPaths::GetPath(EngineConfigDir));
		ProjectRootDir = (ProjectConfigDir.Len() > 0) ? FPaths::GetPath(FPaths::GetPath(ProjectConfigDir)) : FString();

		if (FPaths::IsUnderDirectory(ProjectRootDir, EngineRootDir))
		{
			FString RelativeDir = ProjectRootDir;
			FPaths::MakePathRelativeTo(RelativeDir, *(EngineRootDir + TEXT("/")));
			ProjectNotForLicenseesDir = FPaths::Combine(EngineRootDir, TEXT("Restricted/NotForLicensees"), RelativeDir);
			ProjectNoRedistDir = FPaths::Combine(EngineRootDir, TEXT("Restricted/NoRedist"), RelativeDir);
			ProjectLimitedAccessDir = FPaths::Combine(EngineRootDir, TEXT("Restricted/LimitedAccess"), RelativeDir);
		}
		else
		{
			ProjectNotForLicenseesDir = FPaths::Combine(ProjectRootDir, TEXT("Restricted/NotForLicensees"));
			ProjectNoRedistDir = FPaths::Combine(ProjectRootDir, TEXT("Restricted/NoRedist"));
			ProjectLimitedAccessDir = FPaths::Combine(ProjectRootDir, TEXT("Restricted/LimitedAccess"));
		}
		
		// if we explicitly don't want project configs, then make a limited layer set without any {PROJECT} paths
		if (ProjectConfigDir.Len() == 0 && OverrideLayers.Num() == 0)
		{
			for (const FConfigLayer& Layer : GConfigLayers)
			{
				if (FCString::Strstr(Layer.Path, TEXT("{PROJECT}")) == nullptr)
				{
					OverrideLayers.Add(Layer);
				}
			}
		}
	}
}

FConfigContext& FConfigContext::ResetBaseIni(const TCHAR* InBaseIniName)
{
	// for now, there's nothing that needs to be updated, other than the name here
	BaseIniName = InBaseIniName;

	if (!bDoNotResetConfigFile)
	{
		Branch = nullptr;
	}

	return *this;
}

const FConfigContext::FPerPlatformDirs& FConfigContext::GetPerPlatformDirs(const FString& PlatformName)
{
	FConfigContext::FPerPlatformDirs* Dirs = FConfigContext::PerPlatformDirs.Find(PlatformName);
	if (Dirs == nullptr)
	{
		FString PluginExtDir = "<skip>";
		if (bIsForPlugin)
		{
			// look if there's a plugin extension for this platform, it will have the platform name in the path
			for (const FString& ChildDir : ChildPluginBaseDirs)
			{
				if (ChildDir.Contains(*FString::Printf(TEXT("/%s/"), *PlatformName)))
				{
					PluginExtDir = ChildDir;
					break;
				}
			}
		}
		
		Dirs = &PerPlatformDirs.Emplace(PlatformName, FConfigContext::FPerPlatformDirs
			{
				// PlatformExtensionEngineDir
				FPaths::ConvertPath(EngineRootDir, FPaths::EPathConversion::Engine_PlatformExtension, *PlatformName),
				// PlatformExtensionProjectDir
				FPaths::ConvertPath(ProjectRootDir, FPaths::EPathConversion::Project_PlatformExtension, *PlatformName, *ProjectRootDir),
				// PluginExtensionDir
				PluginExtDir,
			});
	}
	return *Dirs;
}

static FConfigFile& ActiveFile(FConfigContext* Context)
{
	if (Context->ExistingFile) return *Context->ExistingFile;
	if (Context->Branch) return Context->Branch->InMemoryFile;
	unimplemented();
	static FConfigFile Empty;
	return Empty;
}


UE_TRACE_EVENT_BEGIN(Cpu, FConfigContextLoad, NoSync)
UE_TRACE_EVENT_FIELD(UE::Trace::WideString, BaseName)
UE_TRACE_EVENT_END()

bool FConfigContext::Load(const TCHAR* InBaseIniName, FString& OutFinalFilename)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FConfigContext::Load);
#if CPUPROFILERTRACE_ENABLED
	UE_TRACE_LOG_SCOPED_T(Cpu, FConfigContextLoad, CpuChannel)
		<< FConfigContextLoad.BaseName(InBaseIniName);
#endif // CPUPROFILERTRACE_ENABLED

	if (Branch != nullptr && Branch->bIsSafeUnloaded)
	{
		Branch->bIsSafeUnloaded = false;
		return LoadIniFileHierarchy();
	}

	// set up a branch if needed
	if (ExistingFile != nullptr)
	{
		Branch = ExistingFile->Branch;

		// setup a branch one time now, not in Reset
		if (bIsHierarchicalConfig && Branch == nullptr)
		{
			Branch = TemporaryBranch = new FConfigBranch;
			Branch->ReplayMethod = EBranchReplayMethod::NoReplay;
			TemporaryBranch->bIsHierarchical = true;
		}
	}


	if (bCacheOnNextLoad || BaseIniName != InBaseIniName)
	{
		if (bIncludeTagNameInBranchName)
		{
			// prepend the BaseIniName with the Tag if desired
			ResetBaseIni(*(ConfigFileTag.ToString() + InBaseIniName));
		}
		else
		{
			ResetBaseIni(InBaseIniName);
		}
		
		CachePaths();
		bCacheOnNextLoad = false;
	}

	
	// perform short-circuited loading for single file
	if (!bIsHierarchicalConfig)
	{
		return PerformSingleFileLoad();
	}
	
	// find existing branch if we are loading into dynamic layers - we need to do it early in case bIncludeTagNameInBranchName is true, as
	// we will need the original InBaseIniName
	if (bIsForPluginModification)
	{
		Branch = ConfigSystem->FindBranch(InBaseIniName, InBaseIniName);
		// if not found, make one, so we can add dynamic layers to an empty branch
		// note: this is unexpected for now since we are going to be looking in KnownConfigFiles
		if (Branch == nullptr)
		{
			if (DestIniFilename.IsEmpty())
			{
				return false;
			}

			UE_LOGF(LogConfig, Warning, "Unable to find branch %ls, making a new one to read plugin layers into. This isn't expected, tell JoshA if you see this", InBaseIniName);
			Branch = &ConfigSystem->AddNewBranch(DestIniFilename);
		}
	}

	bool bPerformLoad;
	if (!PrepareForLoad(bPerformLoad))
	{
		return false;
	}

	// if we are reloading a known ini file (where OutFinalIniFilename already has a value), then we need to leave the OutFinalFilename alone until we can remove LoadGlobalIniFile completely
	if (OutFinalFilename.Len() > 0 && OutFinalFilename == BaseIniName)
	{
		// do nothing
	}
	else
	{
		check(!bWriteDestIni || !DestIniFilename.IsEmpty());

		OutFinalFilename = DestIniFilename;
	}

	bool bSuccess = true;
	// now load if we need (PrepareForLoad may find an existing file and just use it)
	if (bPerformLoad)
	{
		bSuccess = PerformLoad();
		if (bSuccess && ExistingFile != nullptr && TemporaryBranch != nullptr && Branch->ReplayMethod != EBranchReplayMethod::NoReplay)
		{
			// we need to copy the temporary branch's final result back into the output
			*ExistingFile = TemporaryBranch->InMemoryFile;
		}
		// Unload the branch if it is empty. SafeUnload so that we may re-use the branch should it need to be added to later (e.g. by a plugin)
		if (!bSuccess && ConfigSystem != nullptr && ExistingFile == nullptr && TemporaryBranch == nullptr)
		{
			ConfigSystem->SafeUnloadBranch(InBaseIniName);
		}
	}

	// when we return, the context is likely to go out of scope, so the TempBranch will be deleted, and the 
	// file better not have a reference to it!
	if (ExistingFile && ExistingFile->Branch == TemporaryBranch)
	{
		ExistingFile->Branch = nullptr;
	}
	return bSuccess;
}

bool FConfigContext::Load(const TCHAR* InBaseIniName)
{
	FString Discard;
	return Load(InBaseIniName, Discard);
}

/**
 * This will completely load a single .ini file into the passed in FConfigFile.
 *
 * @param FilenameToLoad - this is the path to the file to
 * @param ConfigFile - This is the FConfigFile which will have the contents of the .ini loaded into
 *
 **/
static bool LoadAnIniFile(const FString& FilenameToLoad, FConfigFile& ConfigFile)
{
	if (!IsUsingLocalIniFile(*FilenameToLoad, nullptr) || DoesConfigFileExistWrapper(*FilenameToLoad))
	{
		ProcessIniContents(*FilenameToLoad, *FilenameToLoad, &ConfigFile, false, false);
		return true;
	}

	//UE_LOGF(LogConfig, Warning, "LoadAnIniFile was unable to find FilenameToLoad: %ls ", *FilenameToLoad);
	return false;
}

bool FConfigContext::PerformSingleFileLoad()
{
	// if the ini name passed in already is a full path, just use it
	if (BaseIniName.EndsWith(TEXT(".ini")))
	{
		DestIniFilename = BaseIniName;
		BaseIniName = FPaths::GetBaseFilename(BaseIniName);
	}
	else
	{
		// generate path to the .ini file (not a Default ini, IniName is the complete name of the file, without path)
		DestIniFilename = FString::Printf(TEXT("%s/%s.ini"), *ProjectConfigDir, *BaseIniName);
	}

	FConfigFile* DestFile;
	
	// if this is for a config system, find/add the branch
	if (ConfigSystem != nullptr)
	{
		Branch = ConfigSystem->FindBranch(*BaseIniName, DestIniFilename);
		
		// if the Branch already exists, then we don't want to load anything unless bForceReload is set
		if (Branch != nullptr)
		{
			if (!bForceReload)
			{
				// already loaded and done, we can stop now
				return true;
			}
		}
		else
		{
			// @todo should we pass in a Name to AddNewBranch? could pass is BaseIniName
			Branch = &ConfigSystem->AddNewBranch(DestIniFilename);
			Branch->bIsHierarchical = false;
		}
		DestFile = &Branch->InMemoryFile;
	}
	else
	{
		DestFile = ExistingFile;
	}
	
	DestFile->Name = FName(*BaseIniName);
	DestFile->PlatformName.Reset();
	DestFile->bHasPlatformName = false;

#if UE_WITH_CONFIG_TRACKING
	FConfigFile& File = ActiveFile(this);
	if (File.LoadType == UE::ConfigAccessTracking::ELoadType::Uninitialized)
	{
		File.LoadType = UE::ConfigAccessTracking::ELoadType::LocalSingleIniFile;
	}
	if (File.LoadType == UE::ConfigAccessTracking::ELoadType::LocalSingleIniFile ||
		File.LoadType == UE::ConfigAccessTracking::ELoadType::ExternalSingleIniFile)
	{
		UE::ConfigAccessTracking::FFile* FileAccess = File.GetFileAccess();
		if (FileAccess)
		{
			FileAccess->OverrideFilenameToLoad = FName(FStringView(DestIniFilename));
		}
	}
#endif

	// load the .ini file straight up
	LoadAnIniFile(*DestIniFilename, *DestFile);

	if (ChangeTracker != nullptr && ChangeTracker->bTrackLoadedFiles)
	{
		UE_LOGF(LogConfig, Display, "Tracking dest ini %ls", *DestIniFilename);
		ChangeTracker->LoadedFiles.Add(DestIniFilename);
	}

	return DestFile->Num() > 0;
}

bool FConfigContext::PrepareForLoad(bool& bPerformLoad)
{
#if !UE_BUILD_SHIPPING
	if (IsInGameThread()) GPrepareForLoadTime -= FPlatformTime::Seconds();
#endif

	checkf(ConfigSystem != nullptr || ExistingFile != nullptr, TEXT("Loading config expects to either have a ConfigFile already passed in, or have a ConfigSystem passed in"));

	// assume we will load, unless some code below determines not to
	bPerformLoad = true;

	// leaving the ability to go back to writing in case we actually find issues - they shouldn't be needed anymore 
	// with all of the FConfigBranch changes
	static bool bAllowWriteDuringLoad = FParse::Param(FCommandLine::Get(), TEXT("writeIniOnLoad"));
	if (!bAllowWriteDuringLoad)
	{
		bWriteDestIni = false;
	}

	// first, make sure the DestIniFilename is set, if needed
	if (bWriteDestIni || bAllowGeneratedIniWhenCooked || FPlatformProperties::RequiresCookedData())
	{
		// delay filling out GeneratedConfigDir because some early configs can be read in that set -savedir, and 
		// FPaths::GeneratedConfigDir() will permanently cache the value
		if (GeneratedConfigDir.IsEmpty())
		{
			// Use the engine directory for editor settings
			if (BaseIniName == TEXT("EditorSettings") ||
				BaseIniName == TEXT("EditorKeyBindings") ||
				BaseIniName == TEXT("EditorLayout"))
			{
				GeneratedConfigDir = FPaths::EngineEditorSettingsDir();
			}
			else
			{
				GeneratedConfigDir = FPaths::GeneratedConfigDir();
			}
		}

		// calculate where this file will be saved/generated to (or at least the key to look up in the ConfigSystem)
		DestIniFilename = FConfigCacheIni::GetDestIniFilename(*BaseIniName, *SavePlatform, *GeneratedConfigDir);
	}
	// if we are reading in another platform's plugin ini files, we need some DestIniFilename to store the Branch with
	else if (bIsForPlugin && !bIsForPluginModification)
	{
		DestIniFilename = BaseIniName + TEXT(".ini");
	}

	// we can re-use (and skip loading) an existing branch/file if:
	//   we are not loading into an existing ConfigFile
	//   we don't want to reload
	//   we found an existing file in the ConfigSystem
	//   the existing file has entries (because Known config files are always going to be found, but they will be empty)
	const bool bLookForExistingBranch = (Branch == nullptr && !bForceReload && ConfigSystem != nullptr);
	if (bLookForExistingBranch)
	{
		Branch = ConfigSystem->FindBranch(*BaseIniName, DestIniFilename);
		if (Branch && Branch->InMemoryFile.Num() > 0)
		{
			// cache off the file just in case something looks in the Context after the Load()
			bPerformLoad = false;
		}
	}

	if (bForceReload)
	{
		// re-use an existing ConfigFile's Engine/Project directories if we have a config system to look in,
		// or no config system and the platform matches current platform (which will look in GConfig)
		if (ConfigSystem != nullptr || (Platform == FPlatformProperties::IniPlatformName() && GConfig != nullptr))
		{
			bool bNeedRecache = false;
			FConfigCacheIni* SearchSystem = ConfigSystem == nullptr ? GConfig : ConfigSystem;
			FConfigBranch* ExistingBranch = SearchSystem->FindBranch(*BaseIniName, DestIniFilename);
			if (ExistingBranch != nullptr)
			{
				if (!ExistingBranch->SourceEngineConfigDir.IsEmpty() && ExistingBranch->SourceEngineConfigDir != EngineConfigDir)
				{
					EngineConfigDir = ExistingBranch->SourceEngineConfigDir;
					bNeedRecache = true;
				}
				if (!ExistingBranch->SourceProjectConfigDir.IsEmpty() && ExistingBranch->SourceProjectConfigDir != ProjectConfigDir)
				{
					ProjectConfigDir = ExistingBranch->SourceProjectConfigDir;
					bNeedRecache = true;
				}
				if (bNeedRecache)
				{
					CachePaths();
				}
			}
		}
	}


	// get or make the Branch to use
	if (Branch == nullptr)
	{
		Branch = ConfigSystem->FindBranch(*BaseIniName, DestIniFilename);
		if (Branch == nullptr)
		{
			check(!DestIniFilename.IsEmpty());
			
			Branch = &ConfigSystem->AddNewBranch(DestIniFilename);

			// we allow plugin branches to get wiped if they had no static layers - non-plugins we keep just in case
			// a plugin wants to modify it (a plugin modifying a plugin would require that the modified plugin has
			// actual config data in it)
			if (bIsForPlugin)
			{
				static bool bRemoveEmptyPluginBranches = !FParse::Param(FCommandLine::Get(), TEXT("NoIniPluginBranchClean"));
				bCanRemoveBranchIfEmpty = bRemoveEmptyPluginBranches;
			}
		}
	}
	Branch->bIsHierarchical = bIsHierarchicalConfig;

	if (IsInGameThread()) GPrepareForLoadTime += FPlatformTime::Seconds();

	return true;
}

bool FConfigContext::PerformLoad()
{
	LLM_SCOPE(ELLMTag::ConfigSystem);

#if !UE_BUILD_SHIPPING
	if (IsInGameThread()) GPerformLoadTime -= FPlatformTime::Seconds();
#endif

#if DISABLE_GENERATED_INI_WHEN_COOKED
	if (BaseIniName == TEXT("GameUserSettings"))
	{
		bAllowGeneratedIniWhenCooked = true;
	}
	else
	{
		// If we asked to disable ini when cooked, disable all ini files except GameUserSettings, which stores user preferences
		bAllowGeneratedIniWhenCooked = false;
		if (FPlatformProperties::RequiresCookedData())
		{
			Branch->InMemoryFile.NoSave = true;
		}
	}
#endif

	FConfigFile& FinalFile = ExistingFile ? *ExistingFile : Branch->InMemoryFile;
#if UE_WITH_CONFIG_TRACKING
	// Set the LoadType before calling GenerateDestIniFile, because it will set it if not
	// already set.
	if (FinalFile.LoadType == UE::ConfigAccessTracking::ELoadType::Uninitialized)
	{
		FinalFile.LoadType = UE::ConfigAccessTracking::ELoadType::LocalIniFile;
	}
#endif
	
	if (bIsForPluginModification)
	{
		// gather the list of files to load (these will become dynamic layers below)
		TArray<FString> GatheredFiles;
		AddStaticLayersToHierarchy(&GatheredFiles);
	
		TArray<FDynamicLayerInfo> Layers;
		for (FString& File : GatheredFiles)
		{
			Layers.Add({ File, ConfigFileTag, (uint16)PluginModificationPriority });
			UE_LOGF(LogConfig, Verbose, "Loading plugin modification file %ls", *File);
		}
	
		// call a function to handle the layers if desired
		if (HandleLayersFunction != nullptr)
		{
			HandleLayersFunction(Layers);
		}
		else
		{
			// now add them all as one operation (optimization to not perform unnecessary duplicated work for each file)
			Branch->AddDynamicLayersToHierarchy(Layers, ChangeTracker, ConfigSystem->StagedGlobalConfigCache, ConfigSystem->StagedPluginConfigCache.Find(ConfigFileTag));
		}
		if (IsInGameThread()) GPerformLoadTime += FPlatformTime::Seconds();
		return true;
	}
	
	if (!bIsFixingUpAfterBinaryConfig)
	{
		// generate the whole standard ini hierarchy
		AddStaticLayersToHierarchy(nullptr);
	}

	// now generate and make sure it's up to date (using IniName as a Base for an ini filename)
	// @todo This bNeedsWrite afaict is always true even if it loaded a completely valid generated/final .ini, and the write below will
	// just write out the exact same thing it read in!
	bool bGeneratedFile = GenerateDestIniFile();

	// if we wiped a branch that had no static files, we cannot continue
	if (bCanRemoveBranchIfEmpty && !bGeneratedFile)
	{
		return false;
	}

	// we are done here!
	if (bIsFixingUpAfterBinaryConfig)
	{
		return true;
	}


	FinalFile.Name = FName(*BaseIniName);
	FinalFile.PlatformName = Platform;
	FinalFile.bHasPlatformName = true;

	// chcek if the config file wants to save all sections
	bool bLocalSaveAllSections = false;
	// Do not report the read of SectionsToSave. Some ConfigFiles are reallocated without it, and reporting
	// logs that the section disappeared. But this log is spurious since if the only reason it was read was
	// for the internal save before the FConfigFile is made publicly available.
	const FConfigSection* SectionsToSaveSection = FinalFile.FindSection(SectionsToSaveString);
	if (SectionsToSaveSection)
	{
		const FConfigValue* Value = SectionsToSaveSection->Find(SaveAllSectionsKey);
		if (Value)
		{
			const FString& ValueStr = Value->GetValueForWriting();
			bLocalSaveAllSections = FCString::ToBool(*ValueStr);
		}
	}

	// we can always save all sections of a User config file, Editor* (not Editor.ini tho, that is already handled in the normal method)
	bool bIsUserFile = BaseIniName.Contains(TEXT("User"));
	bool bIsEditorSettingsFile = BaseIniName.Contains(TEXT("Editor")) && BaseIniName != TEXT("Editor");

	FinalFile.bCanSaveAllSections = bLocalSaveAllSections || bIsUserFile || bIsEditorSettingsFile;

	// don't write anything to disk in cooked builds - we will always use re-generated INI files anyway.
	// Note: Unfortunately bAllowGeneratedIniWhenCooked is often true even in shipping builds with cooked data
	// due to default parameters. We don't dare change this now.
	//
	// Check GIsInitialLoad since no INI changes that should be persisted could have occurred this early.
	// INI changes from code, environment variables, CLI parameters, etc should not be persisted.
	if (!GIsInitialLoad && bWriteDestIni && (!FPlatformProperties::RequiresCookedData() || bAllowGeneratedIniWhenCooked)
		// We shouldn't save config files when in multiprocess mode,
		// otherwise we get file contention in XGE shader builds.
		&& !FParse::Param(FCommandLine::Get(), TEXT("Multiprocess")))
	{
		// Check the config system for any changes made to defaults and propagate through to the saved.
		Branch->InMemoryFile.ProcessSourceAndCheckAgainstBackup();

		// don't write anything out if we are reading into an existing file
		if (bGeneratedFile && ExistingFile == nullptr)
		{
			// if it was dirtied during the above function, save it out now
			FinalFile.Write(DestIniFilename);
		}
	}

#if !UE_BUILD_SHIPPING
	if (IsInGameThread()) GPerformLoadTime += FPlatformTime::Seconds();
#endif

	return bGeneratedFile;
}


/**
 * Allows overriding the (default) .ini file for a given base (ie Engine, Game, etc)
 */
static void ConditionalOverrideIniFilename(FString& IniFilename, const TCHAR* BaseIniName)
{
#if !UE_BUILD_SHIPPING
	// Figure out what to look for on the commandline for an override. Disabled in shipping builds for security reasons
	const FString CommandLineSwitch = FString::Printf(TEXT("DEF%sINI="), BaseIniName);
	FParse::Value(FCommandLine::Get(), *CommandLineSwitch, IniFilename);
#endif
}

static FString PerformBasicReplacements(const FString& InString, const TCHAR* BaseIniName)
{
	FString OutString = InString.Replace(TEXT("{TYPE}"), BaseIniName, ESearchCase::CaseSensitive);
	OutString = OutString.Replace(TEXT("{APPSETTINGS}"), FPlatformProcess::ApplicationSettingsDir(), ESearchCase::CaseSensitive);
	OutString = OutString.Replace(TEXT("{USERSETTINGS}"), FPlatformProcess::UserSettingsDir(), ESearchCase::CaseSensitive);
	OutString = OutString.Replace(TEXT("{USER}"), FPlatformProcess::UserDir(), ESearchCase::CaseSensitive);
	OutString = OutString.Replace(TEXT("{CUSTOMCONFIG}"), *FConfigCacheIni::GetCustomConfigString(), ESearchCase::CaseSensitive);

	return OutString;
}


static FString PerformExpansionReplacements(const FConfigLayerExpansion& Expansion, const FString& InString)
{
	// if there's no replacement to do, the output is just the input
	if (Expansion.Before1 == nullptr)
	{
		return InString;
	}

	// if nothing to replace, then skip it entirely
	if (!InString.Contains(Expansion.Before1) && (Expansion.Before2 == nullptr || !InString.Contains(Expansion.Before2)))
	{
		return TEXT("");
	}

	// replace the directory bits
	FString OutString = InString.Replace(Expansion.Before1, Expansion.After1, ESearchCase::CaseSensitive);
	if (Expansion.Before2 != nullptr)
	{
		OutString = OutString.Replace(Expansion.Before2, Expansion.After2, ESearchCase::CaseSensitive);
	}
	return OutString;
}

FString FConfigContext::PerformFinalExpansions(const FString& InString, const FString& InPlatform)
{
	FString OutString = InString.Replace(TEXT("{ENGINE}"), *EngineRootDir);
	OutString = OutString.Replace(TEXT("{PROJECT}"), *ProjectRootDir);
	OutString = OutString.Replace(TEXT("{RESTRICTEDPROJECT_NFL}"), *ProjectNotForLicenseesDir);
	OutString = OutString.Replace(TEXT("{RESTRICTEDPROJECT_NR}"), *ProjectNoRedistDir);
	OutString = OutString.Replace(TEXT("{RESTRICTEDPROJECT_LA}"), *ProjectLimitedAccessDir);

	if (FPaths::IsUnderDirectory(ProjectRootDir, ProjectNotForLicenseesDir))
	{
		FString RelativeDir = ProjectRootDir;
		FPaths::MakePathRelativeTo(RelativeDir, *(ProjectNotForLicenseesDir + TEXT("/")));

		OutString = OutString.Replace(TEXT("{OPT_SUBDIR}"), *(RelativeDir + TEXT("/")));
	}
	else if (FPaths::IsUnderDirectory(ProjectRootDir, ProjectNoRedistDir))
	{
		FString RelativeDir = ProjectRootDir;
		FPaths::MakePathRelativeTo(RelativeDir, *(ProjectNoRedistDir + TEXT("/")));

		OutString = OutString.Replace(TEXT("{OPT_SUBDIR}"), *(RelativeDir + TEXT("/")));
	}
	else if (FPaths::IsUnderDirectory(ProjectRootDir, ProjectLimitedAccessDir))
	{
		FString RelativeDir = ProjectRootDir;
		FPaths::MakePathRelativeTo(RelativeDir, *(ProjectLimitedAccessDir + TEXT("/")));

		OutString = OutString.Replace(TEXT("{OPT_SUBDIR}"), *(RelativeDir + TEXT("/")));
	}
	else if (FPaths::IsUnderDirectory(ProjectRootDir, EngineRootDir))
	{
		FString RelativeDir = ProjectRootDir;
		FPaths::MakePathRelativeTo(RelativeDir, *(EngineRootDir + TEXT("/")));
		
		OutString = OutString.Replace(TEXT("{OPT_SUBDIR}"), *(RelativeDir + TEXT("/")));
	}
	else
	{
		OutString = OutString.Replace(TEXT("{OPT_SUBDIR}"), TEXT(""));
	}
	
	if (InPlatform.Len() > 0)
	{
		OutString = OutString.Replace(TEXT("{EXTENGINE}"), *GetPerPlatformDirs(InPlatform).PlatformExtensionEngineDir);
		OutString = OutString.Replace(TEXT("{EXTPROJECT}"), *GetPerPlatformDirs(InPlatform).PlatformExtensionProjectDir);
		OutString = OutString.Replace(TEXT("{PLATFORM}"), *InPlatform);
	}

	if (bIsForPlugin)
	{
		OutString = OutString.Replace(TEXT("{PLUGIN}"), *PluginRootDir);
		OutString = OutString.Replace(TEXT("{EXTPLUGIN}"), *GetPerPlatformDirs(InPlatform).PlatformExtensionPluginDir);
	}

	return OutString;
}

void FConfigContext::LogVariables(const TCHAR* InBaseIniName, const FString& InPlatform)
{
	static bool bDumpIniLoadInfo = FParse::Param(FCommandLine::Get(), TEXT("dumpiniloads"));

	if (!bDumpIniLoadInfo)
	{
		return;
	}
	
#define BASIC(x) 	UE_LOGF(LogConfig, Display, "  %s: %ls", #x, *PerformBasicReplacements("{" #x "}", InBaseIniName));
#define EXTRA(x) 	UE_LOGF(LogConfig, Display, "  %s: %ls", #x, *PerformFinalExpansions("{" #x "}", InPlatform));

	UE_LOGF(LogConfig, Display, "Variables for expansion:");
	BASIC(TYPE);
	BASIC(USERSETTINGS);
	BASIC(USER);
	BASIC(CUSTOMCONFIG);
	
	EXTRA(ENGINE);
	EXTRA(PROJECT);
	EXTRA(RESTRICTEDPROJECT_NFL);
	EXTRA(RESTRICTEDPROJECT_NR);
	EXTRA(OPT_SUBDIR);
	EXTRA(EXTENGINE);
	EXTRA(EXTPROJECT);
	EXTRA(PLATFORM);
	EXTRA(PLUGIN);
	EXTRA(EXTPLUGIN);
}


void FConfigContext::AddStaticLayersToHierarchy(TArray<FString>* GatheredLayerFilenames, bool bIsForLogging)
{
	// remember where this file was loaded from
	Branch->SourceEngineConfigDir = EngineConfigDir;
	Branch->SourceProjectConfigDir = ProjectConfigDir;

	// string that can have a reference to it, lower down
	const FString DedicatedServerString = IsRunningDedicatedServer() ? TEXT("DedicatedServer") : TEXT("");

	// cache some platform extension information that can be used inside the loops
	const bool bHasCustomConfig = !FConfigCacheIni::GetCustomConfigString().IsEmpty();


	// figure out what layers and expansions we will want
	EConfigExpansionFlags ExpansionMode = EConfigExpansionFlags::ForUncooked;
	FConfigLayer* Layers = GConfigLayers;
	int32 NumLayers = UE_ARRAY_COUNT(GConfigLayers);
	if (FPlatformProperties::RequiresCookedData() || bIsMakingBinaryConfig)
	{
		ExpansionMode = EConfigExpansionFlags::ForCooked;
	}
	if (bIsForPlugin)
	{
		// this has priority over cooked/uncooked
		ExpansionMode = EConfigExpansionFlags::ForPlugin;
		if (bIsForPluginModification)
		{
			Layers = GPluginModificationLayers;
			NumLayers = UE_ARRAY_COUNT(GPluginModificationLayers);
		}
		else
		{
			Layers = GPluginLayers;
			NumLayers = UE_ARRAY_COUNT(GPluginLayers);
		}
	}
	
	// let the context override the layers if needed
	if (OverrideLayers.Num() > 0)
	{
		Layers = OverrideLayers.GetData();
		NumLayers = OverrideLayers.Num();
	}

	// go over all the config layers
	for (int32 LayerIndex = 0; LayerIndex < NumLayers; LayerIndex++)
	{
		const FConfigLayer& Layer = Layers[LayerIndex];

		// skip optional layers
		if (EnumHasAnyFlags(Layer.Flag, EConfigLayerFlags::RequiresCustomConfig) && !bHasCustomConfig)
		{
			continue;
		}

		// put some info into the key for later use
		int FlagPartOfKey = 0;
		if (EnumHasAnyFlags(Layer.Flag, EConfigLayerFlags::UseGlobalConfigCache))
		{
			FlagPartOfKey |= KeyFlag_UseGlobalCache;
		}
		else if (EnumHasAnyFlags(Layer.Flag, EConfigLayerFlags::UsePluginConfigCache))
		{
			FlagPartOfKey |= KeyFlag_UsePluginCache;
		}
		if (Layer.bHasCheckedExist)
		{
			if (!Layer.bExists)
			{
				continue;
			}
			FlagPartOfKey |= KeyFlag_AssumeExists;
		}
		if (EnumHasAnyFlags(Layer.Flag, EConfigLayerFlags::Deprecated))
		{
			if (Layers == GPluginLayers)
			{
				FlagPartOfKey |= KeyFlag_Deprecated_Plugin;
			}
			else
			{
				FlagPartOfKey |= KeyFlag_Deprecated_Other;
			}
		}

		// start replacing basic variables
		FString LayerPath = PerformBasicReplacements(Layer.Path, *BaseIniName);
		bool bHasPlatformTag = LayerPath.Contains(TEXT("{PLATFORM}"));
		bool bHasEngineTag = LayerPath.StartsWith(TEXT("{ENGINE}"));
		bool bHasProjectTag = !bHasEngineTag && LayerPath.StartsWith(TEXT("{PROJECT}"));
		bool bHasPluginTag = bIsForPlugin && !bHasEngineTag && !bHasProjectTag && LayerPath.StartsWith(TEXT("{PLUGIN}"));

		// expand if it it has {ED} or {EF} expansion tags
		if (!EnumHasAnyFlags(Layer.Flag, EConfigLayerFlags::NoExpand))
		{
			// we assume none of the more special tags in expanded ones
			checkfSlow(FCString::Strstr(Layer.Path, TEXT("{APPSETTINGS}")) == nullptr && FCString::Strstr(Layer.Path, TEXT("{USERSETTINGS}")) == nullptr && FCString::Strstr(Layer.Path, TEXT("{USER}")) == nullptr, TEXT("Expanded config %s shouldn't have {APPSETTINGS} or {USER*} tags in it"), Layer.Path);

			// loop over all the possible expansions
			for (int32 ExpansionIndex = 0; ExpansionIndex < UE_ARRAY_COUNT(GConfigExpansions); ExpansionIndex++)
			{
				// does this expansion match our current mode?
				if ((GConfigExpansions[ExpansionIndex].Flags & ExpansionMode) == EConfigExpansionFlags::None)
				{
					continue;
				}

				FString ExpandedPath = PerformExpansionReplacements(GConfigExpansions[ExpansionIndex], LayerPath);

				// if we didn't replace anything, skip it
				if (ExpandedPath.Len() == 0)
				{
					continue;
				}

				// allow for override, only on BASE EXPANSION!
				if (EnumHasAnyFlags(Layer.Flag, EConfigLayerFlags::AllowCommandLineOverride) && ExpansionIndex == 0)
				{
					checkfSlow(!bHasPlatformTag, TEXT("EConfigLayerFlags::AllowCommandLineOverride config %s shouldn't have a PLATFORM in it"), Layer.Path);

					ConditionalOverrideIniFilename(ExpandedPath, *BaseIniName);
				}

				const FDataDrivenPlatformInfo& Info = FDataDrivenPlatformInfoRegistry::GetPlatformInfo(Platform);

				// go over parents, and then this platform, unless there's no platform tag, then we simply want to run through the loop one time to add it to the
				int32 NumPlatforms = bHasPlatformTag ? Info.IniParentChain.Num() + 1 : 1;
				int32 CurrentPlatformIndex = NumPlatforms - 1;
				int32 DedicatedServerIndex = -1;

				// make DedicatedServer another platform
				if (bHasPlatformTag && IsRunningDedicatedServer())
				{
					NumPlatforms++;
					DedicatedServerIndex = CurrentPlatformIndex + 1;
				}

				for (int PlatformIndex = 0; PlatformIndex < NumPlatforms; PlatformIndex++)
				{
					const FString CurrentPlatform =
						(PlatformIndex == DedicatedServerIndex) ? DedicatedServerString :
						(PlatformIndex == CurrentPlatformIndex) ? Platform :
						Info.IniParentChain[PlatformIndex];

					FString PlatformPath = PerformFinalExpansions(ExpandedPath, CurrentPlatform);

					// @todo restricted - ideally, we would move DedicatedServer files into a directory, like platforms are, but for short term compat,
					// convert the path back to the original (DedicatedServer/DedicatedServerEngine.ini -> DedicatedServerEngine.ini)
					if (PlatformIndex == DedicatedServerIndex)
					{
						PlatformPath.ReplaceInline(TEXT("Config/DedicatedServer/"), TEXT("Config/"));
					}

					// if we match the StartSkippingAtFilename, we are done adding to the hierarchy, so just return
					if (PlatformPath == StartSkippingAtFilename)
					{
						return;
					}
					
					if (PlatformPath.StartsWith(TEXT("<skip>")))
					{
						continue;
					}

					// add this to the list!
					if (GatheredLayerFilenames != nullptr)
					{
						if (bIsForLogging)
						{
							GatheredLayerFilenames->Add(FString::Printf(TEXT("%s[Exp-%d]: %s"), Layer.EditorName, ExpansionIndex, *PlatformPath));
						}
						else
						{
							GatheredLayerFilenames->Add(PlatformPath);
						}
					}
					else
					{
						Branch->Hierarchy.AddStaticLayer(PlatformPath, LayerIndex, ExpansionIndex, PlatformIndex, FlagPartOfKey);
					}
				}
			}
		}
		// if no expansion, just process the special tags (assume no PLATFORM tags)
		else
		{
			checkfSlow(!bHasPlatformTag, TEXT("Non-expanded config %s shouldn't have a PLATFORM in it"), Layer.Path);
			checkfSlow(!EnumHasAnyFlags(Layer.Flag, EConfigLayerFlags::AllowCommandLineOverride), TEXT("Non-expanded config can't have a EConfigLayerFlags::AllowCommandLineOverride"));

			FString FinalPath = PerformFinalExpansions(LayerPath, Platform);

			// if we match the StartSkippingAtFilename, we are done adding to the hierarchy, so just return
			if (FinalPath == StartSkippingAtFilename)
			{
				return;
			}

			// add with no expansion
			if (GatheredLayerFilenames != nullptr)
			{
				if (bIsForLogging)
				{
					GatheredLayerFilenames->Add(FString::Printf(TEXT("%s: %s"), Layer.EditorName, *FinalPath));
				}
				else
				{
					GatheredLayerFilenames->Add(FinalPath);
				}
			}
			else
			{
				Branch->Hierarchy.AddStaticLayer(FinalPath, LayerIndex, 0, 0, FlagPartOfKey);
			}
		}
	}
}


// either show an editor warning, or just write to log for non-editor
// NOTE: For tirst checkin, it will just log a warning, then once we have settled, we can checkin
// more in-your-face behavior
static void ProcessDeprecation(const FString& File, const FText& MessageText)
{
	if (!IsInGameThread())
	{
		AsyncTask(ENamedThreads::GameThread, [File, MessageText]() { ProcessDeprecation(File, MessageText); });
		return;
	}

	FFormatNamedArguments Arguments;
	Arguments.Add(TEXT("File"), FText::FromString(File));
	Arguments.Add(TEXT("Message"), MessageText);
	FText Msg = FText::Format(LOCTEXT("DeprecatedConfigFile", "Found a deprecated '{File}' - {Message}"), Arguments);

	//if (GIsEditor)
	//{
	//	static TSet<FString> AlreadyWarnedKeys;

	//	FString AbsPath = IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*File);

	//	FMessageLog EditorErrors("EditorErrors");
	//	TSharedRef<FTokenizedMessage> Message = EditorErrors.Message(EMessageSeverity::Warning);
	//	if (File.EndsWith(TEXT(".ini")))
	//	{
	//		Message->AddToken(FURLToken::Create(FString::Printf(TEXT("file://%s"), *AbsPath), LOCTEXT("DeprecatedGFPConfig_URLCLick", "Click to open file")));
	//	}
	//	Message->AddToken(FTextToken::Create(Msg));
	//	EditorErrors.Notify();
	//}

	UE_LOGF(LogConfig, Warning, "%ls", *Msg.ToString());
	//ensureAlwaysMsgf(false, TEXT("%s"), *Msg.ToString());
}

/**
 * This will completely load .ini file hierarchy into the passed in FConfigFile. The passed in FConfigFile will then
 * have the data after combining all of those .ini
 *
 * @param FilenameToLoad - this is the path to the file to
 * @param ConfigFile - This is the FConfigFile which will have the contents of the .ini loaded into and Combined()
 *
 **/
bool FConfigContext::LoadIniFileHierarchy()
{
	static bool bDumpIniLoadInfo = FParse::Param(FCommandLine::Get(), TEXT("dumpiniloads"));
	static bool bRemoveMissingLayersFromHierarchy = !FParse::Param(FCommandLine::Get(), TEXT("NoIniHierarchyClean"));

	// LogVariables(*BaseIniName, Platform);

#if ALLOW_INI_OVERRIDE_FROM_COMMANDLINE
	// if we have a commandline overrides, then we need to know now, because we can't 
	// use NoReplay mode safely
	FConfigFile::OverrideFromCommandline(&Branch->CommandLineOverrides, BaseIniName);
	if (Branch->CommandLineOverrides.Num() > 0 && Branch->ReplayMethod == EBranchReplayMethod::NoReplay)
	{
		UE_LOGF(LogConfig, Display, "Branch %ls had commandline override(s), so override NoReplay mode. This will use a bit more memory, but is needed for plugin/GFP handling.", *BaseIniName);
		Branch->ReplayMethod = EBranchReplayMethod::DynamicLayerReplay;
	}
#endif

	bool bReadAnyFile = false;
	TSet<int32> KeysToRemove;

	TRACE_CPUPROFILER_EVENT_SCOPE(LoadIniFileHierarchy);
	// Traverse ini list back to front, merging along the way.
	for (const TPair<int32, FUtf8String>& HierarchyIt : Branch->Hierarchy)
	{
		const int KeyWithoutFlags = HierarchyIt.Key & ~((1 << NumFlagsBits) - 1);
		const FString IniFileName = FString(HierarchyIt.Value);

		const bool bDoCombine = (KeyWithoutFlags != 0);
		const bool bUseStagedGlobalCache = ConfigSystem != nullptr && (HierarchyIt.Key & KeyFlag_UseGlobalCache) != 0;
		const bool bUseStagedPluginCache = (HierarchyIt.Key & KeyFlag_UsePluginCache) != 0;
		const bool bAssumeExists = (HierarchyIt.Key & KeyFlag_AssumeExists) != 0;

		UE_CLOGF(bDumpIniLoadInfo, LogConfig, Display, "Looking for file: %ls", *IniFileName);
		
		// skip non-existant files
		if (!bAssumeExists && IsUsingLocalIniFile(*IniFileName, nullptr) && !DoesConfigFileExistWrapper(*IniFileName, IniCacheSet, bUseStagedGlobalCache ? ConfigSystem->StagedGlobalConfigCache : nullptr, bUseStagedPluginCache ? StagedPluginConfigCache : nullptr))
		{
			if (bRemoveMissingLayersFromHierarchy)
			{
				KeysToRemove.Add(HierarchyIt.Key);
			}
			continue;
		}
		
		if (KeyWithoutFlags != 0)
		{
			bReadAnyFile = true;
		}

		UE_CLOGF(bDumpIniLoadInfo, LogConfig, Display, "   Found %ls!", *IniFileName);

		if (HierarchyIt.Key & KeyFlag_Deprecated_Plugin)
		{
			ProcessDeprecation(IniFileName, LOCTEXT("RenameToDefault", "Rename to Default<PluginName>.ini'"));
		}
		if (HierarchyIt.Key & KeyFlag_Deprecated_Other)
		{
			ProcessDeprecation(IniFileName, LOCTEXT("UnknownConfigDeprecation", "Unknown reason, check ConfigHierarchy.h for EConfigLayerFlags::Deprecated that might trigger this."));
		}

		bool bDoEmptyConfig = false;
		//UE_LOGF(LogConfig, Log,  "Combining configFile: %ls", *IniList(IniIndex) );

		if (Branch->ReplayMethod == EBranchReplayMethod::FullReplay)
		{
			FConfigCommandStream& NewFile = Branch->StaticLayers.Add(IniFileName, FConfigCommandStream());
			NewFile.FillFileFromDisk(IniFileName, bDoCombine);

			// now combine this in to our Static cache
			Branch->CombinedStaticLayers.ApplyFile(&NewFile);
		}
		else if (Branch->ReplayMethod == EBranchReplayMethod::DynamicLayerReplay)
		{
			// apply the file directly info the Static cache
			Branch->CombinedStaticLayers.FillFileFromDisk(IniFileName, bDoCombine);
		}
		else
		{
			// apply the file directly into the InMemory cache
			if (ExistingFile != nullptr)
			{
				ExistingFile->FillFileFromDisk(IniFileName, bDoCombine);
			}
			else
			{
				Branch->InMemoryFile.FillFileFromDisk(IniFileName, bDoCombine);
			}
		}

		if (ChangeTracker != nullptr && ChangeTracker->bTrackLoadedFiles)
		{
			ChangeTracker->LoadedFiles.Add(IniFileName);
		}
	}

	for (int32 Key : KeysToRemove)
	{
		Branch->Hierarchy.Remove(Key);
	}

	// if we had been reading into the Static cache, not InMemory, then start the InMemory from this point
	if (Branch->ReplayMethod != EBranchReplayMethod::NoReplay)
	{
		Branch->CombinedStaticLayers.Shrink();

		Branch->InMemoryFile = Branch->CombinedStaticLayers;
		
		// need to reset this since it just got blown away
		Branch->InMemoryFile.ChangeTracker = &Branch->SavedLayer;
	}
	else
	{
		Branch->InMemoryFile.Shrink();
	}

	Branch->FinalCombinedLayers = Branch->InMemoryFile;

	// look for some standard dynamic layers that we know to look for now (and not safe reloading, which will already handle this
	// since it will reload all dynamic layers already)
	if (!bIsForPlugin && !ProjectConfigDir.IsEmpty() && !bIsSafeReloading)
	{
		FString ProjectPluginsOverride = FPaths::Combine(ProjectConfigDir, FString::Printf(TEXT("PluginOverride%s.ini"), *BaseIniName));

		// use the DynamicLayers version so we can specify the priority
		if (Branch->AddDynamicLayersToHierarchy({ { ProjectPluginsOverride, NAME_None, (uint16)DynamicLayerPriority::ProjectPluginOverrides } }, nullptr, StagedGlobalConfigCache, nullptr))
		{
			bReadAnyFile = true;
		}
	}

	return bReadAnyFile;
}

/**
 * This will load up two .ini files and then determine if the destination one is outdated by comparing
 * version number in [CurrentIniVersion] section, Version key against the version in the Default*.ini
 * Outdatedness is determined by the following mechanic:
 *
 * Outdatedness also can be affected by commandline params which allow one to delete all .ini, have
 * automated build system etc.
 */
bool FConfigContext::GenerateDestIniFile()
{
	if (!bIsFixingUpAfterBinaryConfig)
	{
		// reset the file to empty
		Branch->InMemoryFile.Cleanup();
		Branch->CombinedStaticLayers.Cleanup();
		Branch->SavedLayer.Empty();
		Branch->CommandLineOverrides.Empty();
		Branch->StaticLayers.Empty();
		Branch->DynamicLayers.Empty();
		Branch->FinalCombinedLayers.Empty();

		// read the static files into the branch
		if (!LoadIniFileHierarchy())
		{
			if (bCanRemoveBranchIfEmpty)
			{
				ConfigSystem->Remove(DestIniFilename);
				Branch = TemporaryBranch = nullptr;
				return false;
			}
		}
	}
	
#if !IS_PROGRAM
	// Don't try to load any generated files from disk in cooked builds. We will always use the re-generated INIs.
	// Programs also always want this, so skip the check for Programs
	if (!FPlatformProperties::RequiresCookedData() || bAllowGeneratedIniWhenCooked)
#endif
	{
		if (DestIniFilename.Len() > 0)
		{
			static bool bDumpIniLoadInfo = FParse::Param(FCommandLine::Get(), TEXT("dumpiniloads"));
			UE_CLOGF(bDumpIniLoadInfo, LogConfig, Display, "Looking for saved user ini file: %ls", *DestIniFilename);
			if (DoesConfigFileExistWrapper(*DestIniFilename, nullptr))
			{
				UE_CLOGF(bDumpIniLoadInfo, LogConfig, Display, "   Found!");
				Branch->SavedLayer.FillFileFromDisk(*DestIniFilename, false);
				Branch->SavedLayer.bIsSavedConfigFile = true;
			}
		}
	}
	
	// skip over code that doesn't apply when reading into an ExistingFile
	if (ExistingFile == nullptr && Branch->InMemoryFile.Num() > 0)
	{
		bool bForceRegenerate = false;

		// New versioning
		int32 SourceConfigVersionNum = -1;
		int32 CurrentIniVersion = -1;
		bool bVersionChanged = false;

		// don't do version checking if we have nothing saved
		if (Branch->SavedLayer.Num() > 0)
		{
			// get the version that was last saved, if any
			FConfigCommandStreamSection* VersionSection = Branch->SavedLayer.Find(CurrentIniVersionString);
			if (VersionSection)
			{
				FConfigValue* VersionKey = VersionSection->Find(*VersionName);
				if (VersionKey)
				{
					TTypeFromString<int32>::FromString(CurrentIniVersion, *VersionKey->GetValue());
				}
			}

			// now compare to the source config file
			if (Branch->CombinedStaticLayers.GetInt(*CurrentIniVersionString, *VersionName, SourceConfigVersionNum))
			{
				if (SourceConfigVersionNum > CurrentIniVersion)
				{
					UE_LOGF(LogInit, Log, "%ls version has been updated. It will be regenerated.", *FPaths::ConvertRelativePathToFull(DestIniFilename));
					bVersionChanged = true;
				}
				else if (SourceConfigVersionNum < CurrentIniVersion)
				{
					UE_LOGF(LogInit, Warning, "%ls version is later than the source. Since the versions are out of sync, nothing will be done.", *FPaths::ConvertRelativePathToFull(DestIniFilename));
				}
			}

			// Regenerate the ini file?
			if (FParse::Param(FCommandLine::Get(), TEXT("REGENERATEINIS")) == true)
			{
				bForceRegenerate = true;
			}
		}

		// Order is important, we want to let force regenerate happen before version change, in case we're trying to wipe everything.
		//	Version tries to save some info.
		if (bForceRegenerate)
		{
			Branch->SavedLayer.Empty();
		}
		else if (bVersionChanged)
		{
			// get list of preserved sections (those we want to keep from the Saved, even if the version changed)
			TArray<FString> PreservedSections;
			Branch->InMemoryFile.GetArray(*CurrentIniVersionString, *PreserveName, PreservedSections);

			// get the saved keys, and remove non-preserved ones
			TSet<FString> SavedKeys;
			Branch->SavedLayer.GetKeys(SavedKeys);
			for (const FString& SavedSection : SavedKeys)
			{
				if (!PreservedSections.Contains(SavedSection))
				{
					Branch->SavedLayer.Remove(*SavedSection);
				}
			}

			// make sure current version is saved out (this would only be needed if we preserved the CurrentIniVersionString section, but doesn't hurt to do)
			Branch->SavedLayer.FindOrAdd(CurrentIniVersionString).Remove(*VersionName);
			Branch->SavedLayer.FindOrAdd(CurrentIniVersionString).Add(*VersionName, FConfigValue(FString::Printf(TEXT("%d"), SourceConfigVersionNum), FConfigValue::EValueType::Set));
		}

		// now merge in the saved info that is still around after the above logic
		Branch->InMemoryFile.ApplyFile(&Branch->SavedLayer);

#if ALLOW_INI_OVERRIDE_FROM_COMMANDLINE
		// process any commandline overrides
		FConfigFile::OverrideFromCommandline(&Branch->CommandLineOverrides, BaseIniName);
		// and push it into the current values
		Branch->FinalCombinedLayers.ApplyFile(&Branch->CommandLineOverrides);
		Branch->InMemoryFile.ApplyFile(&Branch->CommandLineOverrides);
#endif

		//	Branch->CombinedStaticLayers.Cleanup();
	}

	// return true if we actually read anything in
	return Branch->InMemoryFile.Num() > 0 || (ExistingFile != nullptr && ExistingFile->Num() > 0);
}




/*-----------------------------------------------------------------------------
	FConfigFileHierarchy
-----------------------------------------------------------------------------*/


FConfigFileHierarchy::FConfigFileHierarchy()
{
}


int32 FConfigFileHierarchy::AddStaticLayer(const FString& Filename, int32 LayerIndex, int32 ExpansionIndex /*= 0*/, int32 PlatformIndex /*= 0*/, int32 Flags)
{
	int32 Key = GetStaticKey(LayerIndex, ExpansionIndex, PlatformIndex, Flags);
	Emplace(Key, FUtf8String(Filename));
	return Key;
}

void FConfigContext::EnsureRequiredGlobalPathsHaveBeenInitialized()
{
	PerformBasicReplacements(TEXT(""), TEXT("")); // requests user directories and FConfigCacheIni::GetCustomConfigString
}


void FConfigContext::VisualizeHierarchy(FOutputDevice& Ar, const TCHAR* IniName, const TCHAR* OverridePlatform, const TCHAR* OverrideProjectOrProgramDataDir, const TCHAR* OverridePluginDir, const TArray<FString>* ChildPluginBaseDirs)
{
	FConfigFile Test;
	FConfigContext Context(nullptr, true, OverridePlatform ? FString(OverridePlatform) : FString(), &Test);
	if (OverridePluginDir != nullptr)
	{
		Context.bIsForPlugin = true;
		Context.PluginRootDir = OverridePluginDir;
		if (ChildPluginBaseDirs != nullptr)
		{
			Context.ChildPluginBaseDirs = *ChildPluginBaseDirs;
		}
	}
	
	if (OverrideProjectOrProgramDataDir != nullptr)
	{
		Context.ProjectConfigDir = FPaths::Combine(OverrideProjectOrProgramDataDir, "Config/");
	}

	
	Context.VisualizeHierarchy(Ar, IniName);
}

void FConfigContext::VisualizeHierarchy(FOutputDevice& Ar, const TCHAR* IniName)
{
	Ar.Logf(TEXT("======================================================="));

	if (bIncludeTagNameInBranchName)
	{
		ResetBaseIni(*(ConfigFileTag.ToString() + IniName));
	}
	else
	{
		ResetBaseIni(IniName);
	}
	CachePaths();
	bool _;
	PrepareForLoad(_);

	Ar.Logf(TEXT("Config hierarchy:"));
	if (ProjectRootDir.Contains(TEXT("/Programs/")))
	{
		Ar.Logf(TEXT("  Program Data Dir: %s"), *ProjectRootDir);
	}
	else
	{
		Ar.Logf(TEXT("  Project Dir: %s"), *ProjectRootDir);
	}
	Ar.Logf(TEXT("  Platform: %s"), *Platform);
	if (bIsForPlugin)
	{
		Ar.Logf(TEXT("  Plugin Root Dir: %s"), *PluginRootDir);
		for (FString& Child : ChildPluginBaseDirs)
		{
			Ar.Logf(TEXT("  Plugin Children Dir: %s"), *Child);
		}
	}
	
	TArray<FString> FileList;
	AddStaticLayersToHierarchy(&FileList, true);
	
	Ar.Logf(TEXT("  Files:"));
	for (const FString& File : FileList)
	{
		Ar.Logf(TEXT("    %s"), *File);
	}

	Ar.Logf(TEXT("======================================================="));
}

#undef LOCTEXT_NAMESPACE
