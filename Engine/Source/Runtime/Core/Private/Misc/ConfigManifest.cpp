// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/ConfigManifest.h"

#include "Containers/Map.h"
#include "Containers/StringConv.h"
#include "Containers/UnrealString.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/PlatformProcess.h"
#include "HAL/PlatformProperties.h"
#include "Misc/App.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/EngineVersion.h"
#include "Misc/EngineVersionBase.h"
#include "Misc/Paths.h"
#include "Runtime/Launch/Resources/Version.h"
#include "Templates/Tuple.h"
#include "Templates/UnrealTemplate.h"

enum class EConfigManifestVersion
{
	/******* DO NOT REMOVE OLD VERSIONS ********/
	Initial,
	RenameEditorAgnosticSettings,
	MigrateProjectSpecificInisToAgnostic,

	// ^ Add new versions above here ^
	NumOfVersions
};

bool IsDirectoryEmpty(const TCHAR* InDirectory)
{
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	struct FVisitor : IPlatformFile::FDirectoryVisitor
	{
		FVisitor() : bHasFiles(false) {}

		bool bHasFiles;
		virtual bool Visit(const TCHAR*, bool bIsDir) override
		{
			if (!bIsDir)
			{
				bHasFiles = true;
				return false;
			}
			return true;
		}

	} Visitor;

	PlatformFile.IterateDirectory(InDirectory, Visitor);

	return !Visitor.bHasFiles;
}

FString ProjectSpecificIniPath(const TCHAR* InLeaf)
{
	return FPaths::GeneratedConfigDir() / ANSI_TO_TCHAR(FPlatformProperties::PlatformName()) / InLeaf;
}

FString ProjectAgnosticIniPath(const TCHAR* InLeaf)
{
	return FPaths::GameAgnosticSavedDir() / TEXT("Config") / ANSI_TO_TCHAR(FPlatformProperties::PlatformName()) / InLeaf;
}

/** Migrates config files from a previous version of the engine. Does nothing on non-installed versions */
void MigratePreviousEngineInis()
{
	if (!FPaths::ShouldSaveToUserDir() && !FApp::IsEngineInstalled())
	{
		// We can't do this in non-installed engines or where we haven't saved to a user directory
		return;
	}

	int32 MinorVersion = FEngineVersion::Current().GetMinor() - 1;

	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	while(MinorVersion >= 0)
	{
		const FEngineVersion PreviousVersion((uint16)FEngineVersion::Current().GetMajor(), (uint16)(MinorVersion--), 0, 0, FString());
	
		const FString Directory = FString(FPlatformProcess::UserSettingsDir()) / VERSION_TEXT(EPIC_PRODUCT_IDENTIFIER) / PreviousVersion.ToString(EVersionComponent::Minor) / TEXT("Saved") / TEXT("Config") / ANSI_TO_TCHAR(FPlatformProperties::PlatformName());
		if (FPaths::DirectoryExists(Directory))
		{
			const FString DestDir = ProjectAgnosticIniPath(TEXT(""));
			if (PlatformFile.CreateDirectoryTree(*DestDir))
			{
				PlatformFile.CopyDirectoryTree(*DestDir, *Directory, false);
			}

			// If we failed to create the directory tree anyway we don't want to allow the possibility of upgrading from even older versions, so early return regardless
			return;
		}
	}
}

void FConfigManifest::UpgradeFromPreviousVersions()
{
	// First off, load the manifest config if it exists
	FConfigFile Manifest;

	const FString ManifestFilename = ProjectAgnosticIniPath(TEXT("Manifest.ini"));

	if (!FPaths::FileExists(ManifestFilename) && IsDirectoryEmpty(*FPaths::GetPath(ManifestFilename)))
	{
		// Copy files from previous versions of the engine, if possible
		MigratePreviousEngineInis();
	}

	const EConfigManifestVersion LatestVersion = (EConfigManifestVersion)((int32)EConfigManifestVersion::NumOfVersions - 1);
	EConfigManifestVersion CurrentVersion = EConfigManifestVersion::Initial;

	if (FPaths::FileExists(ManifestFilename))
	{
		// Load the manifest from the file
		Manifest.Read(*ManifestFilename);

		int64 Version = 0;
		if (Manifest.GetInt64(TEXT("Manifest"), TEXT("Version"), Version) && Version < (int64)EConfigManifestVersion::NumOfVersions)
		{
			CurrentVersion = (EConfigManifestVersion)Version;
		}
	}

	if (CurrentVersion == LatestVersion)
	{
		return;
	}

	CurrentVersion = UpgradeFromVersion(CurrentVersion);

	// Set the version in the manifest, and write it out
	Manifest.SetInt64(TEXT("Manifest"), TEXT("Version"), (int64)CurrentVersion);
	Manifest.Write(ManifestFilename);
}

/** Combine 2 config files together, putting the result in a third */
void CombineConfig(const TCHAR* Base, const TCHAR* Other, const TCHAR* Output)
{
	FConfigFile Config;

	Config.Read(Base);
	Config.Combine(Other);

	Config.Write(Output, false /*bDoRemoteWrite*/);
}

/** Migrate a project specific ini to be a project agnostic one */
void MigrateToAgnosticIni(const TCHAR* SrcIniName, const TCHAR* DstIniName)
{
	const FString OldIni = ProjectSpecificIniPath(SrcIniName);
	const FString NewIni = ProjectAgnosticIniPath(DstIniName);

	if (FPaths::FileExists(OldIni))
	{
		if (!FPaths::FileExists(NewIni))
		{
			IFileManager::Get().Move(*NewIni, *OldIni);
		}
		else
		{
			CombineConfig(*NewIni, *OldIni, *NewIni);
		}
	}
}

/** Migrate a project specific ini to be a project agnostic one */
void MigrateToAgnosticIni(const TCHAR* IniName)
{
	MigrateToAgnosticIni(IniName, IniName);
}

/** Rename an ini file, dealing with the case where the destination already exists */
void RenameIni(const TCHAR* OldIni, const TCHAR* NewIni)
{
	if (FPaths::FileExists(OldIni))
	{
		if (!FPaths::FileExists(NewIni))
		{
			IFileManager::Get().Move(NewIni, OldIni);
		}
		else
		{
			CombineConfig(NewIni, OldIni, NewIni);
		}
	}
}

void FConfigManifest::MigrateEditorUserSettings()
{
	const FString EditorUserSettingsFilename = ProjectSpecificIniPath(TEXT("EditorUserSettings.ini"));
	if (!FPaths::FileExists(EditorUserSettingsFilename))
	{
		return;
	}

	// Handle upgrading editor user settings to the new path
	FConfigFile OldIni;
	OldIni.NoSave = true;
	OldIni.Read(EditorUserSettingsFilename);
	
	if (OldIni.Num() != 0)
	{
		// Rename the config section
		MigrateConfigSection(OldIni, TEXT("/Script/UnrealEd.EditorUserSettings"), TEXT("/Script/UnrealEd.EditorPerProjectUserSettings"));

		const FString EditorPerProjectUserSettingsFilename = ProjectSpecificIniPath(TEXT("EditorPerProjectUserSettings.ini"));

		FConfigFile NewIni;
		NewIni.Read(EditorPerProjectUserSettingsFilename);
		NewIni.AddMissingProperties(OldIni);
		if (!NewIni.Write(EditorPerProjectUserSettingsFilename, false))
		{
			return;
		}
	}

	IFileManager::Get().Move(*(EditorUserSettingsFilename + TEXT(".bak")), *EditorUserSettingsFilename);
}

EConfigManifestVersion FConfigManifest::UpgradeFromVersion(EConfigManifestVersion FromVersion)
{
	// Perform upgrades sequentially...

	if (FromVersion < EConfigManifestVersion::RenameEditorAgnosticSettings)
	{
		// First off, rename the Editor game agnostic ini config to EditorSettings
		auto Path = ProjectAgnosticIniPath(TEXT("EditorSettings.ini"));
		RenameIni(*ProjectAgnosticIniPath(TEXT("EditorGameAgnostic.ini")), *Path);

		FConfigFile EditorSettings;
		EditorSettings.Read(Path);
		MigrateConfigSection(EditorSettings, TEXT("/Script/UnrealEd.EditorGameAgnosticSettings"), TEXT("/Script/UnrealEd.EditorSettings"));
		EditorSettings.Write(Path, false /*bDoRemoteWrite*/);

		FromVersion = EConfigManifestVersion::RenameEditorAgnosticSettings;
	}

	if (FromVersion < EConfigManifestVersion::MigrateProjectSpecificInisToAgnostic)
	{
		if (!FApp::HasProjectName())
		{
			// We can't upgrade game settings if there is no game.
			return FromVersion;
		}

		// The initial versioning made the following changes:

		// 1. Move EditorLayout.ini from Game/Saved/Config to Engine/Saved/Config, thus making it project-agnostic
		// 2. Move EditorKeyBindings.ini from Game/Saved/Config to Engine/Saved/Config, thus making it project-agnostic

		MigrateToAgnosticIni(TEXT("EditorLayout.ini"));
		MigrateToAgnosticIni(TEXT("EditorKeyBindings.ini"));
		
		FromVersion = EConfigManifestVersion::MigrateProjectSpecificInisToAgnostic;
	}

	return FromVersion;
}

void FConfigManifest::MigrateConfigSection(FConfigFile& ConfigFile, const TCHAR* OldSectionName, const TCHAR* NewSectionName)
{
	const FConfigSection* OldSection = ConfigFile.FindSection(OldSectionName);
	if (OldSection)
	{
		const FConfigSection* NewSection = ConfigFile.FindSection(NewSectionName);
		if (NewSection)
		{
			for (auto& Setting : *OldSection)
			{
				if (!NewSection->Contains(Setting.Key))
				{
					ConfigFile.AddToSection(NewSectionName, Setting.Key, Setting.Value.GetSavedValue());
				}
			}
		}
		else
		{
			// Add the new section and remove the old
			FConfigSection SectionCopy = *OldSection;
			ConfigFile.Add(NewSectionName, MoveTemp(SectionCopy));
			ConfigFile.Remove(OldSectionName);
		}
		ConfigFile.Dirty = true;
	}
}
