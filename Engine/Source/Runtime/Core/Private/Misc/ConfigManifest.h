// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

class FConfigFile;

enum class EConfigManifestVersion;

/** Class responsible for upgrading and migrating various config settings. Keeps track of a manifest version in Manifest.ini, stored in engine saved dir. */
class FConfigManifest
{
public:

	/** Perform miscellaneous upgrade of config files. Tracked by the version number stored in Manifest.ini.
	 *	Should be called before any global config initialization to ensure that the relevant files have been upgraded. */
	static void UpgradeFromPreviousVersions();

	/** Migrate what was previously EditorUserSettings.ini to EditorPerProjectUserSettings.ini, if the former exists. */
	static void MigrateEditorUserSettings();

	/** Migrate a config section to a new section, only overwriting entries that don't exist in the new section. */
	static void MigrateConfigSection(FConfigFile& ConfigFile, const TCHAR* OldSection, const TCHAR* NewSection);

private:

	/** Upgrade the config from the specified version, returning the current version after the upgrade (may be the same as FromVersion if upgrade was not possible). */
	static EConfigManifestVersion UpgradeFromVersion(EConfigManifestVersion FromVersion);
};

