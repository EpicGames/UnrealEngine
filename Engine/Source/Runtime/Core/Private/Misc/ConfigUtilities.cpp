// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/ConfigUtilities.h"

#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "HAL/FileManager.h"
#include "HAL/IConsoleManager.h"
#include "Misc/CoreDelegates.h"
#include "Misc/FileHelper.h"
#include "Tasks/Pipe.h"

#if PLATFORM_WRITES_ARE_SLOW
#include "Async/Async.h"
#include "Misc/QueuedThreadPool.h"
#endif

#define UE_HOTFIX_FOR_NEXT_BOOT_FILENAME TEXT("HotfixForNextBoot.txt")



namespace UE::DynamicConfig
{
TMulticastDelegate<void(const FName&, const FName&, class FConfigModificationTracker*)> HotfixPluginForBranch;
TMulticastDelegate<void(const FName&)> RemovePluginHotfixesFromDynamicBranch;
TMulticastDelegate<void(const FConfigModificationTracker* ChangeTracker)> ReloadObjects;
TMulticastDelegate<void(const FConfigModificationTracker* ChangeTracker)> UpdateCVarsAndDeviceProfiles;

void PerformDynamicConfig(FName Tag, TFunction<void(class FConfigModificationTracker*)> PerformModification,
	TFunction<void(class FConfigModificationTracker*)> PostModification)
{
	FConfigModificationTracker ChangeTracker;
	ChangeTracker.bTrackModifiedSections = true;
	
	// run the callback
	PerformModification(&ChangeTracker);
	
	// now update everything if anything was read in!
	if (ChangeTracker.ModifiedSectionsPerBranch.Num() > 0)
	{
		if (ReloadObjects.IsBound())
		{
			// check for any object-reload control in each branch that was modified
			for (const auto& BranchPair : ChangeTracker.ModifiedSectionsPerBranch)
			{
				// @todo: if this ever needs to look in other config systems for some reason, then we'd have to move this
				// code probably into AddDynamicLayerStringToHierarchy/AddDynamicLayersToHierarchy (they could call a helper
				// in ConfigTracker and pass it the branch)
				FString IniNameString = BranchPair.Key.ToString();
				GConfig->GetArray(TEXT("DynamicLayerClassReloadControl"), TEXT("SkipReloadInstances"), ChangeTracker.ClassesToSkipInstances, IniNameString);
				GConfig->GetArray(TEXT("DynamicLayerClassReloadControl"), TEXT("SkipReloadSubclasses"), ChangeTracker.ClassesToSkipSubclasses, IniNameString);
			}

			// reload any config uobjects that were updated (assuming the uobject system is in use)
			ReloadObjects.Broadcast(&ChangeTracker);
		}

		// update any CVars and DPs that were updated
		UpdateCVarsAndDeviceProfiles.Broadcast(&ChangeTracker);
	}
	
	if (PostModification != nullptr)
	{
		PostModification(&ChangeTracker);
	}
	
#if !UE_BUILD_SHIPPING
	
	if (ChangeTracker.bTrackLoadedFiles)
	{
		UE_LOGF(LogConfig, Verbose, "Modified Files:");
		for (FString& File : ChangeTracker.LoadedFiles)
		{
			UE_LOGF(LogConfig, Verbose, "  %ls", *File);
		}
	}
	
	if (ChangeTracker.bTrackModifiedSections)
	{
		UE_LOGF(LogConfig, Verbose, "Modified sections:");
		for (auto Pair : ChangeTracker.ModifiedSectionsPerBranch)
		{
			UE_LOGF(LogConfig, Verbose, "  Branch: %ls", *Pair.Key.ToString());
			for (FString& Section : Pair.Value)
			{
				UE_LOGF(LogConfig, Verbose, "    %ls", *Section);
			}
		}
		
		if (ChangeTracker.CVars.Num() > 0)
		{
			UE_LOGF(LogConfig, Verbose, "Modified Cvars:");
			for (const auto& SectionPair : ChangeTracker.CVars)
			{
				const FConfigModificationTracker::FCVarTracker& CVars = SectionPair.Value;

				UE_LOGF(LogConfig, Verbose, "  Branch: %ls, SetBy Priority: %d", *SectionPair.Key, CVars.CVarPriority);

				// now walk over the updated cvars and set them based on the priority
				for (const auto& TagPair : CVars.CVarEntriesPerBranchPerTag)
				{
					UE_LOGF(LogConfig, Verbose, "    Tag %ls:", *TagPair.Key.ToString());
					for (const auto& BranchPair : TagPair.Value)
					{
						UE_LOGF(LogConfig, Verbose, "      Section %ls:", *BranchPair.Key.ToString());
						const FConfigSection& Section = BranchPair.Value;
						
						for (const auto& CVarPair : Section)
						{
							UE_LOGF(LogConfig, Verbose, "        %ls = %ls", *CVarPair.Key.ToString(), *CVarPair.Value.GetValue());
						}
					}
				}
			}
		}
	}
		 
#endif
}

} // namesapce UE::DynamicConfig


namespace UE::ConfigUtilities
{

UE::Tasks::FPipe AsyncTaskPipe( TEXT("SaveHotfixForNextBootPipe") );

const TCHAR* ConvertValueFromHumanFriendlyValue( const TCHAR* Value )
{
	static const TCHAR* OnValue = TEXT("1");
	static const TCHAR* OffValue = TEXT("0");

	// allow human friendly names
	if (FCString::Stricmp(Value, TEXT("True")) == 0
		|| FCString::Stricmp(Value, TEXT("Yes")) == 0
		|| FCString::Stricmp(Value, TEXT("On")) == 0)
	{
		return OnValue;
	}
	else if (FCString::Stricmp(Value, TEXT("False")) == 0
		|| FCString::Stricmp(Value, TEXT("No")) == 0
		|| FCString::Stricmp(Value, TEXT("Off")) == 0)
	{
		return OffValue;
	}
	return Value;
}

// The dedicated server could have been deployed through cloud provider and allocated dynamically. 
// So can't really save the file for next boot, consider the runtime args for cvar instead
#if !UE_SERVER
void LoadCVarsFromFileForNextBoot(TMap<FString, FString>& OutCVars)
{
	if (!FPaths::HasProjectPersistentDownloadDir())
	{
		UE_LOGF(LogConfig, Log, "No project persistent download dir available for boot hotfix");
		return;
	}

	IFileManager& FileManager = IFileManager::Get();

	const FString FullPath = FPaths::ProjectPersistentDownloadDir() / UE_HOTFIX_FOR_NEXT_BOOT_FILENAME;

	if (!FileManager.FileExists(*FullPath))
	{
		UE_LOGF(LogConfig, Log, "No local boot hotfix file found at: [%ls]", *FullPath);
		return;
	}

	FString Content;
	if (!FFileHelper::LoadFileToString(Content, *FullPath))
	{
		UE_LOGF(LogConfig, Error, "Failed to load local boot hotfix file: [%ls]", *FullPath);
		return;
	}

	// Delete it so that we don't worry about it when write.
	// Also if for some reason the switch don't work well when boot even before getting latest hotfix, 
	// the next boot will likely succeed by default like before without this file
	if (!FileManager.Delete(*FullPath, true/*RequireExists*/))
	{
		UE_LOGF(LogConfig, Error, "Failed to delete local boot hotfix file [%ls]", *FullPath);
	}

	UE_LOGF(LogConfig, Log, "Local boot hotfix file [%ls] loaded and deleted", *FullPath);

	TArray<FString> Lines;
	Content.ParseIntoArrayLines(Lines);

	for (const FString& Line : Lines)
	{
		FString Key;
		FString Value;
		if (ensure(Line.Split(TEXT("="), &Key, &Value)))
		{
			OutCVars.FindOrAdd(Key) = Value;
		}
	}
}
#endif // !UE_SERVER

void SaveCVarForNextBoot(const TCHAR* Key, const TCHAR* Value)
{
#if !UE_SERVER
	if (!FPaths::HasProjectPersistentDownloadDir())
	{
		UE_LOGF(LogConfig, Log, "No persistent download dir, ignoring CVar %ls hotfix for next boot", Key);
		return;
	}

	FString StrKey(Key);
	FString StrValue(Value);

#if PLATFORM_WRITES_ARE_SLOW
	AsyncPool(*GIOThreadPool, [StrKey, StrValue] {
		TMap<FString, FString> CVarsToSave;

		// Read from file, in case there are more than one cvar hotfix event in same run
		LoadCVarsFromFileForNextBoot(CVarsToSave);

		CVarsToSave.FindOrAdd(StrKey) = StrValue;

		FString ContentToSave;
		for (const TPair<FString, FString>& CVarPair : CVarsToSave)
		{
			ContentToSave.Append(FString::Format(TEXT("{0}={1}\r\n"), { CVarPair.Key, CVarPair.Value }));
		}

		const FString FullPath = FPaths::ProjectPersistentDownloadDir() / UE_HOTFIX_FOR_NEXT_BOOT_FILENAME;
		FFileHelper::SaveStringToFile(ContentToSave, *FullPath);

		UE_LOGF(LogConfig, Log, "Local boot hotfix file [%ls] saved with hotfixed CVar: %ls=%ls", *FullPath, *StrKey, *StrValue);
		});
#else
	AsyncTaskPipe.Launch(UE_SOURCE_LOCATION, [StrKey, StrValue] {
		TMap<FString, FString> CVarsToSave;

		// Read from file, in case there are more than one cvar hotfix event in same run
		LoadCVarsFromFileForNextBoot(CVarsToSave);

		CVarsToSave.FindOrAdd(StrKey) = StrValue;

		FString ContentToSave;
		for (const TPair<FString, FString>& CVarPair : CVarsToSave)
		{
			ContentToSave.Append(FString::Format(TEXT("{0}={1}\r\n"), { CVarPair.Key, CVarPair.Value }));
		}

		const FString FullPath = FPaths::ProjectPersistentDownloadDir() / UE_HOTFIX_FOR_NEXT_BOOT_FILENAME;
		FFileHelper::SaveStringToFile(ContentToSave, *FullPath);

		UE_LOGF(LogConfig, Log, "Local boot hotfix file [%ls] saved with hotfixed CVar: %ls=%ls", *FullPath, *StrKey, *StrValue);
	}, LowLevelTasks::ETaskPriority::BackgroundLow);
#endif // PLATFORM_WRITES_ARE_SLOW
#endif // !UE_SERVER
}

#if UE_EDITOR
TMap<FString, FString> CVarsToApplyForDlls;
#endif

void ApplyCVarsFromBootHotfix()
{
#if !UE_SERVER
	TMap<FString, FString> CVarsToApply;
	LoadCVarsFromFileForNextBoot(CVarsToApply);

#if !UE_BUILD_SHIPPING
	// Support passing boot hotfix cvars via the commandline for testing purposes
	FString BootCvars;
	if (FParse::Value(FCommandLine::Get(), TEXT("BootHotfixCvars="), BootCvars))
	{
		TArray<FString> Pairs;
		BootCvars.ParseIntoArray(Pairs, TEXT(","), true);
		for (const FString& Pair : Pairs)
		{
			FString Key, Value;
			if (Pair.Split(TEXT("="), &Key, &Value))
			{
				CVarsToApply.Add(Key, Value);
				UE_LOGF(LogConfig, Log, "Applied commandline boot hotfix cvar %ls=%ls", *Key, *Value);
			}
		}
	}
#endif //!UE_BUILD_SHIPPING

	for (auto It = CVarsToApply.CreateIterator(); It; ++It)
	{
		IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(*It->Key); 
		if (CVar && CVar->TestFlags(ECVF_SaveForNextBoot))
		{
			CVar->Set(*It->Value, ECVF_SetByHotfix);
			It.RemoveCurrent();
		}
	}

#if UE_EDITOR
	if (!CVarsToApply.IsEmpty())
	{
		CVarsToApplyForDlls = MoveTemp(CVarsToApply);
	}
#endif // UE_EDITOR
#endif // !UE_SERVER
}

#if UE_EDITOR
void ApplyCVarOfDllFromBootHotfix(const TCHAR* Key, IConsoleObject* Obj)
{
	FString* Value = CVarsToApplyForDlls.Find(Key);
	if (Value)
	{
		if (IConsoleVariable* CVar = Obj->AsVariable())
		{
			CVar->Set(*(*Value), ECVF_SetByHotfix);
		}
		CVarsToApplyForDlls.Remove(Key);
	}
}
#endif // UE_EDITOR

void OnSetCVarFromIniEntry(const TCHAR *IniFile, const TCHAR *Key, const TCHAR* Value, uint32 SetBy, bool bAllowCheating, bool bNoLogging, FName Tag)
{
	check(IniFile && Key && Value);
	check((SetBy & ECVF_FlagMask) == 0);

	Value = ConvertValueFromHumanFriendlyValue(Value);

	// we don't need to track cvar misses here (a lot will be not found early on in editor builds)
	bool bTrackFrequentCalls = false;
	IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(Key, bTrackFrequentCalls);

	if(CVar)
	{
		bool bCheatFlag = CVar->TestFlags(EConsoleVariableFlags::ECVF_Cheat); 
		
		if(SetBy == ECVF_SetByScalability)
		{
			if(!CVar->TestFlags(EConsoleVariableFlags::ECVF_Scalability) && !CVar->TestFlags(EConsoleVariableFlags::ECVF_ScalabilityGroup))
			{
				FString Msg =  FString::Printf(TEXT("Scalability.ini can only set ECVF_Scalability console variables ('%s'='%s' is ignored)"), Key, Value);
				ensureMsgf(false, TEXT("%s"), *Msg);
				UE_LOGF(LogConfig, Log, "%ls", *Msg);
				return;
			}
		}

		bool bAllowChange = !bCheatFlag || bAllowCheating;

		if(bAllowChange)
		{
#if !NO_LOGGING
			bool BoolValue = CVar->IsVariableBool() ? CVar->GetBool() : false;
			int32 IntValue = CVar->IsVariableInt() ? CVar->GetInt() : 0;
			float FloatValue = CVar->IsVariableFloat() ? CVar->GetFloat() : 0.0f;
			FString StringValue = CVar->IsVariableString() ? CVar->GetString() : FString();
			bool bFirstSet = (CVar->GetFlags() & ECVF_SetByMask) == ECVF_SetByConstructor;
#endif
			if (SetBy == ECVF_SetByMask)
			{
				CVar->SetWithCurrentPriority(Value, Tag);
			}
			else
			{
				CVar->Set(Value, (EConsoleVariableFlags)SetBy, Tag);
			}
#if !NO_LOGGING
			bool bChanged = bFirstSet;
			if(!bChanged)
			{
				if (CVar->IsVariableBool())
				{
					bChanged = BoolValue != CVar->GetBool();
				}
				else if (CVar->IsVariableInt())
				{
					bChanged = IntValue != CVar->GetInt();
				}
				else if (CVar->IsVariableFloat())
				{
					bChanged = FloatValue != CVar->GetFloat();
				}
				else if (CVar->IsVariableString())
				{
					bChanged = StringValue != CVar->GetString();
				}
			}
			UE_CLOGF(!bNoLogging && bChanged, LogConfig, Log, "Set CVar [[%ls:%ls]]", Key, Value);
#endif
		}
		else
		{
#if !DISABLE_CHEAT_CVARS
			if(bCheatFlag)
			{
				// We have one special cvar to test cheating and here we don't want to both the user of the engine
				if(FCString::Stricmp(Key, TEXT("con.DebugEarlyCheat")) != 0)
				{
					ensureMsgf(false, TEXT("The ini file '%s' tries to set the console variable '%s' marked with ECVF_Cheat, this is only allowed in consolevariables.ini"),
						IniFile, Key);
				}
			}
#endif // !DISABLE_CHEAT_CVARS
		}

		if (CVar->TestFlags(ECVF_SaveForNextBoot) && (SetBy == ECVF_SetByHotfix))
		{
			UE_LOGF(LogConfig, Log, "Saving %ls for boot hotfix", Key);
			SaveCVarForNextBoot(Key, Value);
		}
	}
	else
	{
		if (Tag != NAME_None)
		{
			UE_LOGF(LogConfig, Warning, "Setting unregistered CVar %ls with Tag %ls. This will not be unloaded when the Taqg is unloaded, and it won't be usable until registered in code.", Key, *Tag.ToString());
		}
		
		// Create a dummy that is used when someone registers the variable later on.
		// this is important for variables created in external modules, such as the game module
		IConsoleManager::Get().RegisterConsoleVariable(Key, Value, TEXT("IAmNoRealVariable"),
			(uint32)ECVF_Unregistered | (uint32)ECVF_CreatedFromIni | SetBy);
#if !UE_BUILD_SHIPPING
		UE_LOGF(LogConfig, Log, "CVar [[%ls:%ls]] deferred - dummy variable created", Key, Value);
#endif //!UE_BUILD_SHIPPING
	}
}


void ApplyCVarSettingsFromIni(const TCHAR* InSectionName, const TCHAR* InIniFilename, uint32 SetBy, bool bAllowCheating, FName Tag)
{
	FCoreDelegates::OnApplyCVarFromIni.Broadcast(InSectionName, InIniFilename, SetBy, bAllowCheating);

	UE_LOGF(LogConfig,Log,"Applying CVar settings from Section [%ls] File [%ls]",InSectionName,InIniFilename);

	if (const FConfigSection* Section = GConfig->GetSection(InSectionName, false, InIniFilename))
	{
		for (FConfigSectionMap::TConstIterator It(*Section); It; ++It)
		{
			TStringBuilder<128> KeyString;
			It.Key().ToString(KeyString);
			const FString& ValueString = It.Value().GetValue();

			OnSetCVarFromIniEntry(InIniFilename, *KeyString, *ValueString, SetBy, bAllowCheating, false, Tag);
		}
	}
}

void ForEachCVarInSectionFromIni(const TCHAR* InSectionName, const TCHAR* InIniFilename, TFunction<void(IConsoleVariable* CVar, const FString& KeyString, const FString& ValueString)> InEvaluationFunction)
{
	if (const FConfigSection* Section = GConfig->GetSection(InSectionName, false, InIniFilename))
	{
		for (FConfigSectionMap::TConstIterator It(*Section); It; ++It)
		{
			const FString& KeyString = It.Key().ToString();
			const FString& ValueString = ConvertValueFromHumanFriendlyValue(*It.Value().GetValue());

			IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(*KeyString);
			if (CVar)
			{
				InEvaluationFunction(CVar, KeyString, ValueString);
			}
		}
	}
}





class FCVarIniHistoryHelper
{
private:
	struct FCVarIniHistory
	{
		FCVarIniHistory(const FString& InSectionName, const FString& InIniName, uint32 InSetBy, bool InbAllowCheating) :
			SectionName(InSectionName), FileName(InIniName), SetBy(InSetBy), bAllowCheating(InbAllowCheating)
		{}

		FString SectionName;
		FString FileName;
		uint32 SetBy;
		bool bAllowCheating;
	};
	TArray<FCVarIniHistory> CVarIniHistory;
	bool bRecurseCheck;

	void OnApplyCVarFromIniCallback(const TCHAR* SectionName, const TCHAR* IniFilename, uint32 SetBy, bool bAllowCheating)
	{
		if ( bRecurseCheck == true )
		{
			return;
		}
		CVarIniHistory.Add(FCVarIniHistory(SectionName, IniFilename, SetBy, bAllowCheating));
	}


public:
	FCVarIniHistoryHelper() : bRecurseCheck( false )
	{
		FCoreDelegates::OnApplyCVarFromIni.AddRaw(this, &FCVarIniHistoryHelper::OnApplyCVarFromIniCallback);
	}

	~FCVarIniHistoryHelper()
	{
		FCoreDelegates::OnApplyCVarFromIni.RemoveAll(this);
	}

	void ReapplyIniHistory()
	{
		for (const FCVarIniHistory& IniHistory : CVarIniHistory)
		{
			const FString& SectionName = IniHistory.SectionName;
			const FString& IniFilename = IniHistory.FileName;
			const uint32& SetBy = IniHistory.SetBy;
			if (const FConfigSection* Section = GConfig->GetSection(*SectionName, false, *IniFilename))
			{
				for (FConfigSectionMap::TConstIterator It(*Section); It; ++It)
				{
					TStringBuilder<128> KeyString;
					It.Key().ToString(KeyString);
					const FString& ValueString = It.Value().GetValue();

					IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(*KeyString);

					if ( !CVar )
					{
						continue;
					}

					// if this cvar was last set by this config setting 
					// then we want to reapply any new changes
					if (!CVar->TestFlags((EConsoleVariableFlags)SetBy))
					{
						continue;
					}

					// convert to human friendly string
					const TCHAR* HumanFriendlyValue = ConvertValueFromHumanFriendlyValue(*ValueString);
					const FString CurrentValue = CVar->GetString();
					if (CurrentValue.Compare(HumanFriendlyValue, ESearchCase::CaseSensitive) == 0)
					{
						continue;
					}
					else
					{
						// this is more expensive then a CaseSensitive version and much less likely
						if (CurrentValue.Compare(HumanFriendlyValue, ESearchCase::IgnoreCase) == 0)
						{
							continue;
						}
					}

					if (CVar->TestFlags(EConsoleVariableFlags::ECVF_ReadOnly) )
					{
						UE_LOGF(LogConfig, Warning, "Failed to change Readonly CVAR value %ls %ls -> %ls Config %ls %ls",
							*KeyString, *CurrentValue, HumanFriendlyValue, *IniFilename, *SectionName);

						continue;
					}

					UE_LOGF(LogConfig, Display, "Applied changed CVAR value %ls %ls -> %ls Config %ls %ls", 
						*KeyString, *CurrentValue, HumanFriendlyValue, *IniFilename, *SectionName);

					OnSetCVarFromIniEntry(*IniFilename, *KeyString, *ValueString, SetBy, IniHistory.bAllowCheating, false);
				}
			}
		}
		bRecurseCheck=false;
	}
};
TUniquePtr<FCVarIniHistoryHelper> IniHistoryHelper;

#if !UE_BUILD_SHIPPING
class FConfigHistoryHelper
{
private:
	enum class HistoryType : int
	{
		Value,
		Section,
		SectionName,
		Count
	};
	friend const TCHAR* LexToString(HistoryType Type)
	{
		static const TCHAR* Strings[] = {
			TEXT("Value"),
			TEXT("Section"),
			TEXT("SectionName")
		};
		using UnderType = __underlying_type(HistoryType);
		static_assert(static_cast<UnderType>(HistoryType::Count) == UE_ARRAY_COUNT(Strings), "");
		return Strings[static_cast<UnderType>(Type)];
	}

	struct FConfigHistory
	{
		HistoryType Type = HistoryType::Value;

		FString FileName;
		FString SectionName;
		FString Key;
	};
	friend uint32 GetTypeHash(const FConfigHistory& CH)
	{
		uint32 Hash = ::GetTypeHash(CH.Type);
		Hash = HashCombine(Hash, GetTypeHash(CH.FileName));
		Hash = HashCombine(Hash, GetTypeHash(CH.SectionName));
		Hash = HashCombine(Hash, GetTypeHash(CH.Key));
		return Hash;
	}
	friend bool operator==(const FConfigHistory& A, const FConfigHistory& B)
	{
		return
			A.Type == B.Type &&
			A.FileName == B.FileName &&
			A.SectionName == B.SectionName &&
			A.Key == B.Key;
	}

	TSet<FConfigHistory> History;

	void OnConfigValueRead(const TCHAR* FileName, const TCHAR* SectionName, const TCHAR* Key)
	{
		History.Emplace(FConfigHistory{ HistoryType::Value, FileName, SectionName, Key });
	}

	void OnConfigSectionRead(const TCHAR* FileName, const TCHAR* SectionName)
	{
		History.Emplace(FConfigHistory{ HistoryType::Section, FileName, SectionName });
	}

	void OnConfigSectionNameRead(const TCHAR* FileName, const TCHAR* SectionName)
	{
		History.Emplace(FConfigHistory{ HistoryType::SectionName, FileName, SectionName });
	}

public:
	FConfigHistoryHelper()
	{
		FCoreDelegates::TSOnConfigValueRead().AddRaw(this, &FConfigHistoryHelper::OnConfigValueRead);
		FCoreDelegates::TSOnConfigSectionRead().AddRaw(this, &FConfigHistoryHelper::OnConfigSectionRead);
		FCoreDelegates::TSOnConfigSectionNameRead().AddRaw(this, &FConfigHistoryHelper::OnConfigSectionNameRead);
	}

	~FConfigHistoryHelper()
	{
		FCoreDelegates::TSOnConfigValueRead().RemoveAll(this);
		FCoreDelegates::TSOnConfigSectionRead().RemoveAll(this);
		FCoreDelegates::TSOnConfigSectionNameRead().RemoveAll(this);
	}

	void DumpHistory()
	{
		const FString SavePath = FPaths::ProjectLogDir() / TEXT("ConfigHistory.csv");

		FArchive* Writer = IFileManager::Get().CreateFileWriter(*SavePath, FILEWRITE_NoFail);

		auto WriteLine = [Writer](FString&& Line)
		{
			UE_LOGF(LogConfig, Display, "%ls", *Line);
			FTCHARToUTF8 UTF8String(*(MoveTemp(Line) + LINE_TERMINATOR));
			Writer->Serialize((UTF8CHAR*)UTF8String.Get(), UTF8String.Length());
		};

		UE_LOGF(LogConfig, Display, "Dumping History of Config Reads to %ls", *SavePath);
		UE_LOGF(LogConfig, Display, "Begin History of Config Reads");
		UE_LOGF(LogConfig, Display, "------------------------------------------------------");
		WriteLine(FString::Printf(TEXT("Type, File, Section, Key")));
		for (const FConfigHistory& CH : History)
		{
			switch (CH.Type)
			{
			case HistoryType::Value:
				WriteLine(FString::Printf(TEXT("%s, %s, %s, %s"), LexToString(CH.Type), *CH.FileName, *CH.SectionName, *CH.Key));
				break;
			case HistoryType::Section:
			case HistoryType::SectionName:
				WriteLine(FString::Printf(TEXT("%s, %s, %s, None"), LexToString(CH.Type), *CH.FileName, *CH.SectionName));
				break;
			}
		}
		UE_LOGF(LogConfig, Display, "------------------------------------------------------");
		UE_LOGF(LogConfig, Display, "End History of Config Reads");

		delete Writer;
		Writer = nullptr;
	}
};
TUniquePtr<FConfigHistoryHelper> ConfigHistoryHelper;
#endif // !UE_BUILD_SHIPPING


void RecordApplyCVarSettingsFromIni()
{
	check(IniHistoryHelper == nullptr);
	IniHistoryHelper = MakeUnique<FCVarIniHistoryHelper>();
}

void ReapplyRecordedCVarSettingsFromIni()
{
	// first we need to reload the inis 
	for (const FString& Filename : GConfig->GetFilenames())
	{
		FConfigFile* ConfigFile = GConfig->FindConfigFile(Filename);
		if (ConfigFile->Num() > 0)
		{
			FName BaseName = ConfigFile->Name;
			// Must call LoadLocalIniFile (NOT LoadGlobalIniFile) to preserve original enginedir/sourcedir for plugins
			verify(FConfigCacheIni::LoadLocalIniFile(*ConfigFile, *BaseName.ToString(), true, nullptr, true));
		}
	}

	check(IniHistoryHelper != nullptr);
	IniHistoryHelper->ReapplyIniHistory();
}

void DeleteRecordedCVarSettingsFromIni()
{
	check(IniHistoryHelper != nullptr);
	IniHistoryHelper = nullptr;
}

void RecordConfigReadsFromIni()
{
#if !UE_BUILD_SHIPPING
	check(ConfigHistoryHelper == nullptr);
	ConfigHistoryHelper = MakeUnique<FConfigHistoryHelper>();
#endif
}

void DumpRecordedConfigReadsFromIni()
{
#if !UE_BUILD_SHIPPING
	check(ConfigHistoryHelper);
	ConfigHistoryHelper->DumpHistory();
#endif
}

void DeleteRecordedConfigReadsFromIni()
{
#if !UE_BUILD_SHIPPING
	check(ConfigHistoryHelper);
	ConfigHistoryHelper = nullptr;
#endif
}

}
