// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/CoreAsyncTaskNotificationImpl.h"
#include "Misc/ScopeLock.h"
#include "Logging/LogMacros.h"
#include "Internationalization/Internationalization.h"

#define LOCTEXT_NAMESPACE "CoreAsyncTaskNotification"

FCoreAsyncTaskNotificationImpl::FCoreAsyncTaskNotificationImpl()
{
	checkf(IsInGameThread(), TEXT("AsyncTaskNotification must be constructed on the game thread before being optionally passed to another thread."));
}

FCoreAsyncTaskNotificationImpl::~FCoreAsyncTaskNotificationImpl()
{
	checkf(State != EAsyncTaskNotificationState::Pending, TEXT("AsyncTaskNotification was still pending when destroyed. Missing call to SetComplete?"));
}

void FCoreAsyncTaskNotificationImpl::Initialize(const FAsyncTaskNotificationConfig& InConfig)
{
	FScopeLock Lock(&SynchronizationObject);

	TitleText = InConfig.TitleText;
	ProgressText = InConfig.ProgressText;
	LogCategory = InConfig.LogCategory;

	UpdateNotification();
}

void FCoreAsyncTaskNotificationImpl::SetTitleText(const FText& InTitleText, const bool bClearProgressText)
{
	FScopeLock Lock(&SynchronizationObject);

	TitleText = InTitleText;
	if (bClearProgressText)
	{
		ProgressText = FText();
	}

	UpdateNotification();
}

void FCoreAsyncTaskNotificationImpl::SetProgressText(const FText& InProgressText)
{
	FScopeLock Lock(&SynchronizationObject);

	ProgressText = InProgressText;
	UpdateNotification();
}

void FCoreAsyncTaskNotificationImpl::SetPromptText(const FText& InPromptText)
{
	FScopeLock Lock(&SynchronizationObject);

	PromptText = InPromptText;
	UpdateNotification();
}

void FCoreAsyncTaskNotificationImpl::SetHyperlink(const FSimpleDelegate& InHyperlink, const FText& InHyperlinkText)
{
	FScopeLock Lock(&SynchronizationObject);

	Hyperlink = InHyperlink;
	HyperlinkText = InHyperlinkText;

	UpdateNotification();
}

void FCoreAsyncTaskNotificationImpl::SetComplete(const bool bSuccess)
{
	FScopeLock Lock(&SynchronizationObject);

	State = bSuccess ? EAsyncTaskNotificationState::Success : EAsyncTaskNotificationState::Failure;

	UpdateNotification();
}

void FCoreAsyncTaskNotificationImpl::SetComplete(const FText& InTitleText, const FText& InProgressText, const bool bSuccess)
{
	FAsyncNotificationStateData StateData(InTitleText, InProgressText, bSuccess ? EAsyncTaskNotificationState::Success : EAsyncTaskNotificationState::Failure);
	StateData.PromptText = PromptText;
	StateData.HyperlinkText = HyperlinkText;
	StateData.Hyperlink = Hyperlink;
	SetNotificationState(StateData);
}


void FCoreAsyncTaskNotificationImpl::SetNotificationState(const FAsyncNotificationStateData& InState)
{
	FScopeLock Lock(&SynchronizationObject);
	bool bUpdateNotification = !TitleText.IdenticalTo(InState.TitleText)
		|| !ProgressText.IdenticalTo(InState.ProgressText)
		|| !PromptText.IdenticalTo(InState.PromptText)
		|| !HyperlinkText.IdenticalTo(InState.HyperlinkText)
		|| State != InState.State;

	TitleText = InState.TitleText;
	ProgressText = InState.ProgressText;
	PromptText = InState.PromptText;
	HyperlinkText = InState.HyperlinkText;
	Hyperlink = InState.Hyperlink;
	State = InState.State;

	if (bUpdateNotification)
	{
		UpdateNotification();
	}
}

void FCoreAsyncTaskNotificationImpl::SetCanCancel(const TAttribute<bool>& InCanCancel)
{
}

void FCoreAsyncTaskNotificationImpl::SetKeepOpenOnSuccess(const TAttribute<bool>& InKeepOpenOnSuccess)
{
}

void FCoreAsyncTaskNotificationImpl::SetKeepOpenOnFailure(const TAttribute<bool>& InKeepOpenOnFailure)
{
}

EAsyncTaskNotificationPromptAction FCoreAsyncTaskNotificationImpl::GetPromptAction() const
{
	return EAsyncTaskNotificationPromptAction::Unattended;
}

void FCoreAsyncTaskNotificationImpl::UpdateNotification()
{
	LogNotification();
}

void FCoreAsyncTaskNotificationImpl::LogNotification()
{
#if !NO_LOGGING
	const ELogVerbosity::Type LogVerbosity = State == EAsyncTaskNotificationState::Failure ? ELogVerbosity::Error : ELogVerbosity::Log;
	if (LogCategory && !LogCategory->IsSuppressed(LogVerbosity))
	{
		FString NotificationMessage;
		NotificationMessage += TitleText.ToString();
		if (!ProgressText.IsEmpty())
		{
			if (!NotificationMessage.IsEmpty())
			{
				NotificationMessage += TEXT(" - ");
			}
			NotificationMessage += ProgressText.ToString();
		}
		if (!HyperlinkText.IsEmpty())
		{
			if (!NotificationMessage.IsEmpty())
			{
				NotificationMessage += TEXT(" - ");
			}
			NotificationMessage += HyperlinkText.ToString();
		}
		if (!PromptText.IsEmpty())
		{
			if (!NotificationMessage.IsEmpty())
			{
				NotificationMessage += TEXT(" - ");
			}
			NotificationMessage += PromptText.ToString();
		}

		if (!NotificationMessage.IsEmpty())
		{
			static const FText PendingStateText = LOCTEXT("NotificationState_Pending", "Pending");
			static const FText SuccessStateText = LOCTEXT("NotificationState_Success", "Success");
			static const FText FailureStateText = LOCTEXT("NotificationState_Failure", "Failure");
			static const FText PromptStateText = LOCTEXT("NotificationState_Prompt", "Prompt");

			FText StateText = PendingStateText;
			switch (State)
			{
			case EAsyncTaskNotificationState::Success:
				StateText = SuccessStateText;
				break;
			case EAsyncTaskNotificationState::Failure:
				StateText = FailureStateText;
				break;
			case EAsyncTaskNotificationState::Prompt:
				StateText = PromptStateText;
				break;
			// default:
			// nothing
			}
			FMsg::Logf(nullptr, 0, LogCategory->GetCategoryName(), LogVerbosity, TEXT("[%s] %s"), *StateText.ToString(), *NotificationMessage);
		}
	}
#endif
}

FAsyncTaskNotificationFactory& FAsyncTaskNotificationFactory::Get()
{
	static FAsyncTaskNotificationFactory Factory;
	return Factory;
}

void FAsyncTaskNotificationFactory::RegisterFactory(const FName InName, const FFactoryFunc& InFunc)
{
	UnregisterFactory(InName);
	Factories.Add(MakeTuple(InName, InFunc));
}

void FAsyncTaskNotificationFactory::UnregisterFactory(const FName InName)
{
	Factories.RemoveAll([InName](const TTuple<FName, FFactoryFunc>& FactoryPair)
	{
		return FactoryPair.Key == InName;
	});
}

FAsyncTaskNotificationFactory::FImplPointerType FAsyncTaskNotificationFactory::InvokeFactory() const
{
	return Factories.Num() > 0
		? Factories.Last().Value()
		: MakeShared<FCoreAsyncTaskNotificationImpl>();
}

#undef LOCTEXT_NAMESPACE
