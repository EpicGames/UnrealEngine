// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/CoreDelegates.h"

#include "Misc/CoreDelegatesInternal.h"
#include "Misc/Fork.h"
#include "Math/Vector.h"

//////////////////////////////////////////////////////////////////////////
// FCoreDelegates

TArray<TDelegate<void(void*, int32)>> FCoreDelegates::HotFixDelegates;
TArray<TDelegate<bool(const FString&, FString&)>> FCoreDelegates::PackageNameResolvers;
TMulticastDelegate<void(const TArrayView<UObject*>)> FCoreDelegates::CleanupUnloadingObjects;

TDelegate<void(void*, int32)>& FCoreDelegates::GetHotfixDelegate(EHotfixDelegates::Type HotFix)
{
	if (HotFix >= HotFixDelegates.Num())
	{
		HotFixDelegates.SetNum(HotFix + 1);
	}
	return HotFixDelegates[HotFix];
}

TMulticastDelegate<void()>& FCoreDelegates::GetPreMainInitDelegate()
{
	static TMulticastDelegate<void()> StaticDelegate;
	return StaticDelegate;
}

TDelegate<int32(const TArray<FString>&)> FCoreDelegates::OnMountAllPakFiles;
TDelegate<IPakFile*(const FString&, int32)> FCoreDelegates::MountPak;
TDelegate<bool(TArrayView<UE::FMountPaksExArgs>)> FCoreDelegates::MountPaksEx;
TDelegate<bool(const FString&)> FCoreDelegates::OnUnmountPak;
TDelegate<int32(const FString&)> FCoreDelegates::PakChunkIdOverrideDelegate;

TDelegate<void()> FCoreDelegates::OnOptimizeMemoryUsageForMountedPaks;

TTSMulticastDelegate<void(const IPakFile&)>& FCoreDelegates::GetOnPakFileMounted2()
{
	static TTSMulticastDelegate<void(const IPakFile&)> Singleton;
	return Singleton;
}

TTSMulticastDelegate<void(const IPakFile&)>& FCoreDelegates::GetOnPakFileUnmounting()
{
	static TTSMulticastDelegate<void(const IPakFile&)> Singleton;
	return Singleton;
}

TTSMulticastDelegate<void(const IPakFile&)>& FCoreDelegates::GetOnPakFileUnmounted()
{
	static TTSMulticastDelegate<void(const IPakFile&)> Singleton;
	return Singleton;
}

TMulticastDelegate<void(const FString&)> FCoreDelegates::NewFileAddedDelegate;
TMulticastDelegate<void()> FCoreDelegates::NoPakFilesMountedDelegate;
TMulticastDelegate<void(const FString&)> FCoreDelegates::PakFileMountFailedDelegate;

TTSMulticastDelegate<void(const TCHAR*, const TCHAR*)>& FCoreDelegates::GetOnFileOpenedForReadFromPakFile()
{
	static TTSMulticastDelegate<void(const TCHAR*, const TCHAR*)> Singleton;
	return Singleton;
}

//namespace UE::Core
//{

FMountOperationPak& FCoreInternalDelegates::GetOnPakMountOperation()
{
	static FMountOperationPak Delegate;
	return Delegate;
}

FCurrentlyMountedPaksDelegate& FCoreInternalDelegates::GetCurrentlyMountedPaksDelegate()
{
	static FCurrentlyMountedPaksDelegate Delegate;
	return Delegate;
}

//} // namespace UE::Core

TMulticastDelegate<void(bool, int32, int32)> FCoreDelegates::OnUserLoginChangedEvent;
TMulticastDelegate<void()> FCoreDelegates::OnSafeFrameChangedEvent;
TMulticastDelegate<void(const ANSICHAR*, const ANSICHAR*, int32, const TCHAR*, const TCHAR*)> FCoreDelegates::OnEnsureFailed;
TMulticastDelegate<void()> FCoreDelegates::OnHandleSystemEnsure;
TMulticastDelegate<void()> FCoreDelegates::OnHandleSystemError;
TMulticastDelegate<void()> FCoreDelegates::OnHandleSystemHang;

TMulticastDelegate<void(AActor*)> FCoreDelegates::OnActorLabelChanged;

TMulticastDelegate<void(FCoreDelegates::FMovieStreamerPtr)> FCoreDelegates::RegisterMovieStreamerDelegate;
TMulticastDelegate<void(FCoreDelegates::FMovieStreamerPtr)> FCoreDelegates::UnRegisterMovieStreamerDelegate;

TMulticastDelegate<void(const FGuid&, const FAES::FAESKey&)>& FCoreDelegates::GetRegisterEncryptionKeyMulticastDelegate()
{
	static TMulticastDelegate<void(const FGuid&, const FAES::FAESKey&)> RegisterEncryptionKeyDelegate;
	return RegisterEncryptionKeyDelegate;
}

TDelegate<void(uint8[32])>& FCoreDelegates::GetPakEncryptionKeyDelegate()
{
	static TDelegate<void(uint8[32])> PakEncryptionKeyDelegate;
	return PakEncryptionKeyDelegate;
}

TDelegate<void(TArray<uint8>&, TArray<uint8>&)>& FCoreDelegates::GetPakSigningKeysDelegate()
{
	static TDelegate<void(TArray<uint8>&, TArray<uint8>&)> PakSigningKeysDelegate;
	return PakSigningKeysDelegate;
}

#if WITH_EDITOR
	FSimpleMulticastDelegate FCoreDelegates::PreModal;
	FSimpleMulticastDelegate FCoreDelegates::PostModal;
	FCoreDelegates::FPrePostSlateModalWithContextDelegate FCoreDelegates::PreSlateModalWithContext;
	FCoreDelegates::FPrePostSlateModalWithContextDelegate FCoreDelegates::PostSlateModalWithContext;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
    FSimpleMulticastDelegate FCoreDelegates::PreSlateModal;
    FSimpleMulticastDelegate FCoreDelegates::PostSlateModal;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif	//WITH_EDITOR
#if ALLOW_OTHER_PLATFORM_CONFIG
	TDelegate<FCoreDelegates::FCVarKeyValueMap(const FString&)> FCoreDelegates::GatherDeviceProfileCVars;
#endif
FSimpleMulticastDelegate FCoreDelegates::OnShutdownAfterError;
FSimpleMulticastDelegate FCoreDelegates::OnInit;
FSimpleMulticastDelegate FCoreDelegates::OnOutputDevicesInit;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
FSimpleMulticastDelegate FCoreDelegates::OnPostEngineInit;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
FSimpleMulticastDelegate FCoreDelegates::OnAllModuleLoadingPhasesComplete;
FSimpleMulticastDelegate FCoreDelegates::OnFEngineLoopInitComplete;
FSimpleMulticastDelegate FCoreDelegates::OnCommandletPreMain;
FSimpleMulticastDelegate FCoreDelegates::OnCommandletPostMain;
FSimpleMulticastDelegate FCoreDelegates::OnExit;
FSimpleMulticastDelegate FCoreDelegates::OnPreExit;
FSimpleMulticastDelegate FCoreDelegates::OnEnginePreExit;
TMulticastDelegate<void(TArray<FString>&)> FCoreDelegates::GatherAdditionalLocResPathsCallback;
FSimpleMulticastDelegate FCoreDelegates::ColorPickerChanged;
FSimpleMulticastDelegate FCoreDelegates::OnBeginFrame;
FSimpleMulticastDelegate FCoreDelegates::OnSamplingInput;
FSimpleMulticastDelegate FCoreDelegates::OnEndFrame;
FSimpleMulticastDelegate FCoreDelegates::OnBeginFrameRT;
FSimpleMulticastDelegate FCoreDelegates::OnEndFrameRT;
FCoreDelegates::FOnNetworkConnectionChanged FCoreDelegates::OnNetworkConnectionChanged;
TDelegate<EAppReturnType::Type(EAppMsgCategory, EAppMsgType::Type, const FText&, const FText&)> FCoreDelegates::ModalMessageDialog;
TMulticastDelegate<void(const FString&, const FString&)> FCoreDelegates::OnInviteAccepted;
TMulticastDelegate<void(class UWorld*, FIntVector, FIntVector)> FCoreDelegates::PreWorldOriginOffset;
TMulticastDelegate<void(class UWorld*, FIntVector, FIntVector)> FCoreDelegates::PostWorldOriginOffset;
TDelegate<void()> FCoreDelegates::StarvedGameLoop;
TMulticastDelegate<void(FCoreDelegates::ETemperatureSeverity)> FCoreDelegates::OnTemperatureChange;
TMulticastDelegate<void(bool)> FCoreDelegates::OnLowPowerMode;

TMulticastDelegate<void()> FCoreDelegates::ApplicationWillDeactivateDelegate;
TMulticastDelegate<void()> FCoreDelegates::ApplicationHasReactivatedDelegate;
TMulticastDelegate<void()> FCoreDelegates::ApplicationWillEnterBackgroundDelegate;
TMulticastDelegate<void()> FCoreDelegates::ApplicationHasEnteredForegroundDelegate;

TTSMulticastDelegate<void()>& FCoreDelegates::GetApplicationWillTerminateDelegate()
{
	static TTSMulticastDelegate<void()> Singleton;
	return Singleton;
}

TMulticastDelegate<void()> FCoreDelegates::ApplicationShouldUnloadResourcesDelegate;
TMulticastDelegate<void(float)> FCoreDelegates::MobileBackgroundTickDelegate;

TMulticastDelegate<void(bool)> FCoreDelegates::ApplicationSystemUIOverlayStateChangedDelegate;

TMulticastDelegate<void(const TArray<FString>&)> FCoreDelegates::ApplicationReceivedStartupArgumentsDelegate;

TMulticastDelegate<void(bool)> FCoreDelegates::UserMusicInterruptDelegate;
TMulticastDelegate<void(bool)> FCoreDelegates::AudioRouteChangedDelegate;
TMulticastDelegate<void(bool)> FCoreDelegates::AudioInterruptionDelegate;
TMulticastDelegate<void(bool, int)> FCoreDelegates::AudioMuteDelegate;
TMulticastDelegate<void()> FCoreDelegates::ApplicationRequestAudioState;

TMulticastDelegate<void(TArray<uint8>)> FCoreDelegates::ApplicationRegisteredForRemoteNotificationsDelegate;
TMulticastDelegate<void(int)> FCoreDelegates::ApplicationRegisteredForUserNotificationsDelegate;
TMulticastDelegate<void(FString)> FCoreDelegates::ApplicationFailedToRegisterForRemoteNotificationsDelegate;
TMulticastDelegate<void(FString, int)> FCoreDelegates::ApplicationReceivedRemoteNotificationDelegate;
TMulticastDelegate<void(FString, int, int)> FCoreDelegates::ApplicationReceivedLocalNotificationDelegate;

TMulticastDelegate<void()> FCoreDelegates::ApplicationPerformFetchDelegate;
TMulticastDelegate<void(FString)> FCoreDelegates::ApplicationBackgroundSessionEventDelegate;

TMulticastDelegate<void(bool)> FCoreDelegates::AudioFocusChangedDelegate;

FSimpleMulticastDelegate& FCoreDelegates::GetOnPostEngineInit()
{
	static FSimpleMulticastDelegate Singleton;
	return Singleton;
}

TTSMulticastDelegate<void(const TCHAR*, int32&)>& FCoreDelegates::TSCountPreLoadConfigFileRespondersDelegate()
{
	static TTSMulticastDelegate<void(const TCHAR*, int32&)> Singleton;
	return Singleton;
}

TTSMulticastDelegate<void(const TCHAR*, FString&)>& FCoreDelegates::TSPreLoadConfigFileDelegate()
{
	static TTSMulticastDelegate<void(const TCHAR*, FString&)> Singleton;
	return Singleton;
}

TTSMulticastDelegate<void(const TCHAR*, const FString&, int32&)>& FCoreDelegates::TSPreSaveConfigFileDelegate()
{
	static TTSMulticastDelegate<void(const TCHAR*, const FString&, int32&)> Singleton;
	return Singleton;
}

TTSMulticastDelegate<void(const TCHAR*, bool&)>& FCoreDelegates::TSPreDeleteConfigFileDelegate()
{
	static TTSMulticastDelegate<void(const TCHAR*, bool&)> Singleton;
	return Singleton;
}

TTSMulticastDelegate<void(const FConfigFile*)>& FCoreDelegates::TSOnFConfigCreated()
{
	static TTSMulticastDelegate<void(const FConfigFile*)> Singleton;
	return Singleton;
}

TTSMulticastDelegate<void(const FConfigFile*)>& FCoreDelegates::TSOnFConfigDeleted()
{
	static TTSMulticastDelegate<void(const FConfigFile*)> Singleton;
	return Singleton;
}

TTSMulticastDelegate<void(const TCHAR*, const TCHAR*, const TCHAR*)>& FCoreDelegates::TSOnConfigValueRead()
{
	static TTSMulticastDelegate<void(const TCHAR*, const TCHAR*, const TCHAR*)> Singleton;
	return Singleton;
}

TTSMulticastDelegate<void(const TCHAR*, const TCHAR*)>& FCoreDelegates::TSOnConfigSectionRead()
{
	static TTSMulticastDelegate<void(const TCHAR*, const TCHAR*)> Singleton;
	return Singleton;
}

TTSMulticastDelegate<void(const TCHAR*, const TCHAR*)>& FCoreDelegates::TSOnConfigSectionNameRead()
{
	static TTSMulticastDelegate<void(const TCHAR*, const TCHAR*)> Singleton;
	return Singleton;
}

TTSMulticastDelegate<void(const FString&, const TSet<FString>&)>& FCoreDelegates::TSOnConfigSectionsChanged()
{
	static TTSMulticastDelegate<void(const FString&, const TSet<FString>&)> Singleton;
	return Singleton;
}

TTSMulticastDelegate<void(const TCHAR*)>& FCoreDelegates::TSOnConfigBranchUnloaded()
{
	static TTSMulticastDelegate<void(const TCHAR*)> Singleton;
	return Singleton;
}

TTSMulticastDelegate<void(const TCHAR*, float)>& FCoreDelegates::TSOnConfigBranchReloaded()
{
	static TTSMulticastDelegate<void(const TCHAR*, float)> Singleton;
	return Singleton;
}

TMulticastDelegate<void(const TCHAR*, const TCHAR*, uint32, bool)> FCoreDelegates::OnApplyCVarFromIni;
TMulticastDelegate<void(uint32, uint32)> FCoreDelegates::OnSystemResolutionChanged;

#if WITH_EDITOR
TMulticastDelegate<void(const ITargetPlatform*)> FCoreDelegates::OnTargetPlatformChangedSupportedFormats;
TMulticastDelegate<void(int, const FName&)> FCoreDelegates::OnFeatureLevelDisabled;
#endif 

TMulticastDelegate<void(const TCHAR*, bool&, bool&)> FCoreDelegates::StatCheckEnabled;
TMulticastDelegate<void(const TCHAR*)> FCoreDelegates::StatEnabled;
TMulticastDelegate<void(const TCHAR*)> FCoreDelegates::StatDisabled;
TMulticastDelegate<void(const bool)> FCoreDelegates::StatDisableAll;

TMulticastDelegate<void()> FCoreDelegates::ApplicationLicenseChange;
TTSMulticastDelegate<void()> FCoreDelegates::ApplicationHeartbeat;

TMulticastDelegate<void(EConvertibleLaptopMode)> FCoreDelegates::PlatformChangedLaptopMode;

TMulticastDelegate<void()> FCoreDelegates::VRHeadsetTrackingInitializingAndNeedsHMDToBeTrackedDelegate;
TMulticastDelegate<void()> FCoreDelegates::VRHeadsetTrackingInitializedDelegate;
TMulticastDelegate<void()> FCoreDelegates::VRHeadsetRecenter;
TMulticastDelegate<void()> FCoreDelegates::VRHeadsetLost;
TMulticastDelegate<void()> FCoreDelegates::VRHeadsetReconnected;
TMulticastDelegate<void()> FCoreDelegates::VRHeadsetConnectCanceled;
TMulticastDelegate<void()> FCoreDelegates::VRHeadsetPutOnHead;
TMulticastDelegate<void()> FCoreDelegates::VRHeadsetRemovedFromHead;
TMulticastDelegate<void()> FCoreDelegates::VRControllerRecentered;

TMulticastDelegate<void(const FString&)> FCoreDelegates::UserActivityStringChanged;
TMulticastDelegate<void(const FString&)> FCoreDelegates::GameSessionIDChanged;
TMulticastDelegate<void(const FString&)> FCoreDelegates::GameStateClassChanged;
TMulticastDelegate<void(const FCrashOverrideParameters&)> FCoreDelegates::CrashOverrideParamsChanged;
TMulticastDelegate<void(bool)> FCoreDelegates::IsVanillaProductChanged;

TMulticastDelegate<void()> FCoreDelegates::OnAsyncLoadingFlush;
TMulticastDelegate<void()> FCoreDelegates::OnAsyncLoadingFlushUpdate;

TTSMulticastDelegate<void(FStringView)>& FCoreDelegates::GetOnAsyncLoadPackage()
{
	static TTSMulticastDelegate<void(FStringView)> Singleton;
	return Singleton;
}

TMulticastDelegate<void(const FString&)> FCoreDelegates::OnSyncLoadPackage;
TMulticastDelegate<void()> FCoreDelegates::PostRenderingThreadCreated;
TMulticastDelegate<void()> FCoreDelegates::PreRenderingThreadDestroyed;

TMulticastDelegate<void(int32)> FCoreDelegates::ApplicationReceivedScreenOrientationChangedNotificationDelegate;

TTSMulticastDelegate<void()>& FCoreDelegates::TSConfigReadyForUse()
{
	static TTSMulticastDelegate<void()> Singleton;
	return Singleton;
}

TDelegate<bool()> FCoreDelegates::IsLoadingMovieCurrentlyPlaying;

TDelegate<bool(const TCHAR*)> FCoreDelegates::ShouldLaunchUrl;
TDelegate<void(const FString&, FString*)> FCoreDelegates::LaunchCustomHandlerForURL;

TMulticastDelegate<void(const FString&, FPlatformUserId)> FCoreDelegates::OnActivatedByProtocol;

TMulticastDelegate<void(const FString&)> FCoreDelegates::OnGCFinishDestroyTimeExtended;

TTSMulticastDelegate<void(FCoreDelegates::FExtraBinaryConfigData&)>& FCoreDelegates::TSAccessExtraBinaryConfigData()
{
	static TTSMulticastDelegate<void(FCoreDelegates::FExtraBinaryConfigData&)> Singleton;
	return Singleton;
}


TDelegate<void(TArrayView<const FShaderHash>, FCoreDelegates::FIoChunkIdResolvedFunc)> FCoreDelegates::ResolvePackageShaderMaps;
TDelegate<void(TArrayView<const FShaderHash>, FCoreDelegates::FAttachShaderReadRequestFunc)> FCoreDelegates::PreloadPackageShaderMaps;
TDelegate<void(TArrayView<const FShaderHash>)> FCoreDelegates::ReleasePreloadedPackageShaderMaps;
TMulticastDelegate<void(const FLogCategoryName&, ELogVerbosity::Type, ELogVerbosity::Type)> FCoreDelegates::OnLogVerbosityChanged;

TMulticastDelegate<void(bool)> FCoreDelegates::ApplicationNetworkInitializationChanged;
TMulticastDelegate<void(ENetworkConnectionStatus, ENetworkConnectionStatus)> FCoreDelegates::OnNetworkConnectionStatusChanged;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FCrashOverrideParameters::~FCrashOverrideParameters()
{
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

/**	 Implemented as a function to address global ctor issues */
FSimpleMulticastDelegate& FCoreDelegates::GetMemoryTrimDelegate()
{
	static FSimpleMulticastDelegate OnMemoryTrim;
	return OnMemoryTrim;
}

FSimpleMulticastDelegate& FCoreDelegates::GetLowLevelAllocatorMemoryTrimDelegate()
{
	static FSimpleMulticastDelegate OnLowLevelAllocatorMemoryTrim;
	return OnLowLevelAllocatorMemoryTrim;
}

FSimpleMulticastDelegate& FCoreDelegates::GetRefreshLowLevelAllocatorDelegate()
{
	static FSimpleMulticastDelegate OnRefreshLowLevelAllocator;
	return OnRefreshLowLevelAllocator;
}

/**	 Implemented as a function to address global ctor issues */
FSimpleMulticastDelegate& FCoreDelegates::GetOutOfMemoryDelegate()
{
	static FSimpleMulticastDelegate OnOOM;
	return OnOOM;
}

/**	 Implemented as a function to address global ctor issues */
TMulticastDelegate<void(const uint64, const uint64)>& FCoreDelegates::GetGPUOutOfMemoryDelegate()
{
	static TMulticastDelegate<void(const uint64, const uint64)> OnGPUOOM;
	return OnGPUOOM;
}

TMulticastDelegate<void(FCoreDelegates::FSeverityMessageMap&)> FCoreDelegates::OnGetOnScreenMessages;

typedef void(*TSigningKeyFunc)(TArray<uint8>&, TArray<uint8>&);
typedef void(*TEncryptionKeyFunc)(unsigned char[32]);

CORE_API void RegisterSigningKeyCallback(TSigningKeyFunc InCallback)
{
	FCoreDelegates::GetPakSigningKeysDelegate().BindLambda([InCallback](TArray<uint8>& OutExponent, TArray<uint8>& OutModulus)
	{
		InCallback(OutExponent, OutModulus);
	});
}

CORE_API void RegisterEncryptionKeyCallback(TEncryptionKeyFunc InCallback)
{
	FCoreDelegates::GetPakEncryptionKeyDelegate().BindLambda([InCallback](uint8 OutKey[32])
	{
		InCallback(OutKey);
	});
}

TDelegate<const TCHAR*(void)> FCoreDelegates::OnGetBuildURL;
TDelegate<const TCHAR*(void)> FCoreDelegates::OnGetExecutingJobURL;

#if WITH_EDITOR
TMulticastDelegate<void(const UE::FMultiprocessCreatedContext&)> FCoreDelegates::OnMultiprocessWorkerCreated;
TMulticastDelegate<void(const UE::FMultiprocessDetachedContext&)> FCoreDelegates::OnMultiprocessWorkerDetached;
#endif

FSimpleMulticastDelegate FCoreDelegates::OnParentBeginFork;
FSimpleMulticastDelegate FCoreDelegates::OnParentPreFork;
TMulticastDelegate<void(EForkProcessRole /* ProcessRole */)> FCoreDelegates::OnPostFork;
FSimpleMulticastDelegate FCoreDelegates::OnChildEndFramePostFork;
FCoreDelegates::FOnDisallowedExecCommandCalled FCoreDelegates::OnDisallowedExecCommandCalled;
FCoreDelegates::FOnNamedCommandParsed FCoreDelegates::OnNamedCommandParsed;

FSimpleMulticastDelegate FCoreDelegates::OnLowLevelMemTrackerTick;
