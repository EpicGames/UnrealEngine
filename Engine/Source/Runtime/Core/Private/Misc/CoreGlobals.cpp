// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreGlobals.h"
#include "Internationalization/Text.h"
#include "Internationalization/Internationalization.h"
#include "Stats/Stats.h"
#include "HAL/IConsoleManager.h"
#include "Logging/StructuredLog.h"
#include "Modules/ModuleManager.h"
#include "Misc/CoreStats.h"
#include "Misc/TrackedActivity.h"
#include "Misc/Compression.h"
#include "Misc/CoreDelegates.h"
#include "Misc/LazySingleton.h"
#include "Misc/PlayInEditorLoadingScope.h"
#include "Misc/CommandLine.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "ProfilingDebugging/MiscTrace.h"
#include "GenericPlatform/GenericPlatformCrashContext.h"
#include "CoreGlobalsInternal.h"

#ifndef FAST_PATH_UNIQUE_NAME_GENERATION
#define FAST_PATH_UNIQUE_NAME_GENERATION (!WITH_EDITORONLY_DATA)
#endif

#ifndef UE_PROJECT_NAME
#define UE_PROJECT_NAME None
#define UE_IS_GAME_AGNOSTIC true
#else
#define UE_IS_GAME_AGNOSTIC false
#endif


#define LOCTEXT_NAMESPACE "Core"

CSV_DEFINE_CATEGORY(EngineInit, true);

class FCoreModule : public FDefaultModuleImpl
{
public:
	virtual bool SupportsDynamicReloading() override
	{
		// Core cannot be unloaded or reloaded
		return false;
	}
};


IMPLEMENT_MODULE( FCoreModule, Core );


/*-----------------------------------------------------------------------------
	Global variables.
-----------------------------------------------------------------------------*/
CORE_API FFeedbackContext*	GWarn					= nullptr;				/* User interaction and non critical warnings */
FConfigCacheIni*			GConfig					= nullptr;				/* Configuration database cache */
ITransaction*				GUndo					= nullptr;				/* Transaction tracker, non-NULL when a transaction is in progress */
FOutputDeviceConsole*		GLogConsole				= nullptr;				/* Console log hook */
CORE_API FMalloc**			GFixedMallocLocationPtr = nullptr;				/* Memory allocator pointer location when PLATFORM_USES_FIXED_GMalloc_CLASS is true */
#if USING_THREAD_SANITISER || USING_INSTRUMENTATION
CORE_API TSAN_ATOMIC(FMalloc*) UE::Private::GMalloc;						/* Memory allocator */
#else
CORE_API FMalloc*			UE::Private::GMalloc = nullptr;				/* Memory allocator */
#endif
CORE_API FMalloc* const&	GMalloc	                = UE::Private::GMalloc;	/* Provides backwards compatibility for external users of un-namespaced GMalloc. */

class FPropertyWindowManager*	GPropertyWindowManager	= nullptr;		/* Manages and tracks property editing windows */

/** For building call stack text dump in guard/unguard mechanism. */
TCHAR GErrorHist[16384] = {};

/** For building exception description text dump in guard/unguard mechanism. */
TCHAR GErrorExceptionDescription[4096] = {};

// We define our texts like this so that the header only needs to refer to references to FTexts,
// as FText is only forward-declared there.
struct FCoreTextsSingleton
{
	FCoreTextsSingleton()
		: True (LOCTEXT("True",  "True"))
		, False(LOCTEXT("False", "False"))
		, Yes  (LOCTEXT("Yes",   "Yes"))
		, No   (LOCTEXT("No",    "No"))
		, None (LOCTEXT("None",  "None"))
		, Texts{
			True,
			False,
			Yes,
			No,
			None
		}
	{
	}

	FText True;
	FText False;
	FText Yes;
	FText No;
	FText None;

	FCoreTexts Texts;
};

const FCoreTexts& FCoreTexts::Get()
{
	return TLazySingleton<FCoreTextsSingleton>::Get().Texts;
}

void FCoreTexts::TearDown()
{
	TLazySingleton<FCoreTextsSingleton>::TearDown();
}

/** If true, this executable is able to run all games (which are loaded as DLL's) **/
#if UE_GAME || UE_SERVER
	// In monolithic builds, implemented by the IMPLEMENT_GAME_MODULE macro or by UnrealGame module.
	#if !IS_MONOLITHIC
		bool GIsGameAgnosticExe = UE_IS_GAME_AGNOSTIC;
	#endif
#else
	// In monolithic Editor builds, implemented by the IMPLEMENT_GAME_MODULE macro or by UnrealGame module.
	#if !IS_MONOLITHIC || !UE_EDITOR
		// Otherwise only modular editors are game agnostic.
		#if IS_PROGRAM || IS_MONOLITHIC
			bool GIsGameAgnosticExe = false;
		#else
			bool GIsGameAgnosticExe = UE_IS_GAME_AGNOSTIC;
		#endif
	#endif //!IS_MONOLITHIC || !UE_EDITOR
#endif

/** When saving out of the game, this override allows the game to load editor only properties **/
bool GForceLoadEditorOnly = false;

/** Name of the core package					**/
FLazyName GLongCorePackageName(TEXT("/Script/Core"));
/** Name of the core package					**/
FLazyName GLongCoreUObjectPackageName(TEXT("/Script/CoreUObject"));

/** Disable loading of objects not contained within script files; used during script compilation */
bool GVerifyObjectReferencesOnly = false;

/** when constructing objects, use the fast path on consoles... */
#if FAST_PATH_UNIQUE_NAME_GENERATION && !WITH_EDITOR
bool GFastPathUniqueNameGeneration = true;
#else
bool GFastPathUniqueNameGeneration = false;
#endif

/** allow AActor object to execute script in the editor from specific entry points, such as when running a construction script */
bool GAllowActorScriptExecutionInEditor = false;

/** Forces use of template names for newly instanced components in a CDO */
bool GCompilingBlueprint = false;

/** True if we're garbage collecting after a blueprint compilation */
bool GIsGCingAfterBlueprintCompile = false;

/** True if we're reconstructing blueprint instances. Should never be true on cooked builds */
bool GIsReconstructingBlueprintInstances = false;

/** True if actors and objects are being re-instanced. */
std::atomic<bool> GIsReinstancing = false;

/** Settings for when using UE as a library */
FUELibraryOverrideSettings GUELibraryOverrideSettings;

/**
 * If true, we are running an editor script that should not prompt any dialog modal. The default value of any model will be used.
 * This is used when running an editor utility blueprint or script like Python and we don't want an OK dialog to pop while the script is running.
 * Could be set for commandlet with -RUNNINGUNATTENDEDSCRIPT
 */
bool GIsRunningUnattendedScript = false;

#if WITH_EDITOR
bool					PRIVATE_GIsRunningCookCommandlet	= false;				/** Whether this executable is running the cook commandlet */
bool					PRIVATE_GIsRunningDLCCookCommandlet = false;				/** Whether this executable is running the cook commandlet on a DLC plugin */
bool					PRIVATE_GIsRunningHybridCookedEditor= false;				/** Whether the editor is running in hybrid cooked editor mode */
bool					PRIVATE_GHasInitializedHybridCookedEditor = false;			/** Whether PRIVATE_GIsRunningHybridCookedEditor has been initialized */
namespace UE::Private
{
int32					GMultiprocessId = 0;
}
#endif

#if WITH_ENGINE
bool					PRIVATE_GIsRunningCommandlet		= false;				/** Whether this executable is running a commandlet (custom command-line processing code) */
UClass*					PRIVATE_GRunningCommandletClass		= nullptr;				/** Class of running cook commandlet */
bool					PRIVATE_GAllowCommandletRendering	= false;				/** If true, initialise RHI and set up scene for rendering even when running a commandlet. */
bool					PRIVATE_GAllowCommandletAudio 		= false;				/** If true, allow audio even when running a commandlet. */
#endif	// WITH_ENGINE

#if WITH_EDITORONLY_DATA
bool					GIsEditor						= false;					/* Whether engine was launched for editing */
bool					GIsImportingT3D					= false;					/* Whether editor is importing T3D */
bool					GIsTransacting					= false;					/* true if there is an undo/redo operation in progress. */
bool					GIntraFrameDebuggingGameThread	= false;					/* Indicates that the game thread is currently paused deep in a call stack; do not process any game thread tasks */
bool					GFirstFrameIntraFrameDebugging	= false;					/* Indicates that we're currently processing the first frame of intra-frame debugging */
#elif USING_CODE_ANALYSIS
// These are always false during 'non-editor code analysis', just like they would be when #defined.
bool					GIsEditor						= false;
bool					GIntraFrameDebuggingGameThread	= false;
bool					GFirstFrameIntraFrameDebugging	= false;
#endif // !WITH_EDITORONLY_DATA

bool					GEdSelectionLock				= false;					/* Are selections locked? (you can't select/deselect additional actors) */
bool					GIsClient						= false;					/* Whether engine was launched as a client */
bool					GIsServer						= false;					/* Whether engine was launched as a server, true if GIsClient */
TSAN_ATOMIC(bool)		GIsCriticalError				= false;					/* An appError() has occured */
bool					GIsGuarded						= false;					/* Whether execution is happening within main()/WinMain()'s try/catch handler */
TSAN_ATOMIC(bool)		GIsRunning(false);											/* Whether execution is happening within MainLoop() */
FIsDuplicatingClassForReinstancing	GIsDuplicatingClassForReinstancing;					        /* Whether we are currently using SDO on a UClass or CDO for live reinstancing */
/** This specifies whether the engine was launched as a build machine process								*/
bool					GIsBuildMachine					= false;
/** This determines if we should output any log text.  If Yes then no log text should be emitted.			*/
bool					GIsSilent						= false;
bool					GIsSlowTask						= false;					/* Whether there is a slow task in progress */
bool					GSlowTaskOccurred				= false;					/* Whether a slow task began last tick*/
bool					GIsRequestingExit				= false;					/* Indicates that MainLoop() should be exited at the end of the current iteration */
/** Archive for serializing arbitrary data to and from memory												*/
bool					GAreScreenMessagesEnabled		= true;						/* Whether onscreen warnings/messages are enabled */
bool					GScreenMessagesRestoreState		= false;					/* Used to restore state after a screenshot */
int32					GIsDumpingMovie					= 0;						/* Whether we are dumping screenshots (!= 0), exposed as console variable r.DumpingMovie */
bool					GIsHighResScreenshot			= false;					/* Whether we're capturing a high resolution shot */
uint32					GScreenshotResolutionX			= 0;						/* X Resolution for high res shots */
uint32					GScreenshotResolutionY			= 0;						/* Y Resolution for high res shots */
uint64					GMakeCacheIDIndex				= 0;						/* Cache ID */

FString				GEngineIni;													/* Engine ini filename */

/** Editor ini file locations - stored per engine version (shared across all projects). Migrated between versions on first run. */
FString				GEditorIni;													/* Editor ini filename */
FString				GEditorKeyBindingsIni;										/* Editor Key Bindings ini file */
FString				GEditorLayoutIni;											/* Editor UI Layout ini filename */
FString				GEditorSettingsIni;											/* Editor Settings ini filename */

/** Editor per-project ini files - stored per project. */
FString				GEditorPerProjectIni;										/* Editor User Settings ini filename */

FString				GCompatIni;
FString				GLightmassIni;												/* Lightmass settings ini filename */
FString				GScalabilityIni;											/* Scalability settings ini filename */
FString				GHardwareIni;												/* Hardware ini filename */
FString				GInputIni;													/* Input ini filename */
FString				GGameIni;													/* Game ini filename */
FString				GGameUserSettingsIni;										/* User Game Settings ini filename */
FString				GRuntimeOptionsIni;											/* Runtime Options ini filename */
FString				GInstallBundleIni;											/* Install Bundle ini filename*/
FString				GDeviceProfilesIni;											/* Runtime DeviceProfiles ini filename - use LoadLocalIni for other platforms' DPs */
FString				GGameplayTagsIni;											/* Gameplay tags for the GameplayTagManager */

float				GNearClippingPlane					= 10.0f;				/* Near clipping plane */
float				GNearClippingPlane_RenderThread		= 10.0f;				/* Near clipping plane (Render Thread accessible) */

bool					GExitPurge						= false;

CORE_API FChunkedFixedUObjectArray* GCoreObjectArrayForDebugVisualizers = nullptr;

namespace UE::CoreUObject::Private
{
	struct FStoredObjectPathDebug;
	struct FObjectHandlePackageDebugData;
}
UE::CoreUObject::Private::FStoredObjectPathDebug* GCoreComplexObjectPathDebug = nullptr;
UE::CoreUObject::Private::FObjectHandlePackageDebugData* GCoreObjectHandlePackageDebug = nullptr;
UE::CoreUObject::Private::FStoreBase* const* GCoreFixedTagStoresForDebugVisualizers = nullptr;
#if PLATFORM_UNIX || PLATFORM_MAC
uint8** CORE_API GNameBlocksDebug = FNameDebugVisualizer(FClangKeepDebugInfo{}).GetBlocks();
FChunkedFixedUObjectArray*& CORE_API GObjectArrayForDebugVisualizers = GCoreObjectArrayForDebugVisualizers;
UE::CoreUObject::Private::FStoredObjectPathDebug*& GComplexObjectPathDebug = GCoreComplexObjectPathDebug;
UE::CoreUObject::Private::FObjectHandlePackageDebugData*& CORE_API GObjectHandlePackageDebug = GCoreObjectHandlePackageDebug;
#endif

/** Game name, used for base game directory and ini among other things										*/
#if (!IS_MONOLITHIC && !IS_PROGRAM)
// In modular game builds, the game name will be set when the application launches
TCHAR					GInternalProjectName[64]					= TEXT(UE_STRINGIZE(UE_PROJECT_NAME));
#elif !IS_MONOLITHIC && IS_PROGRAM
// In non-monolithic programs builds, the game name will be set by the module, but not just yet, so we need to NOT initialize it!
TCHAR					GInternalProjectName[64];
#else
// For monolithic builds, the game name variable definition will be set by the IMPLEMENT_GAME_MODULE
// macro for the game's main game module.
#endif

// Foreign engine directory. This is required to projects built outside the engine root to reference their engine directory.
#if !IS_MONOLITHIC
IMPLEMENT_FOREIGN_ENGINE_DIR()
#endif

/** A function that does nothing. Allows for a default behavior for callback function pointers. */
static void appNoop()
{
}

std::atomic<bool> GEngineStartupModuleLoadingComplete = false;
CORE_API bool IsEngineStartupModuleLoadingComplete()
{
	return GEngineStartupModuleLoadingComplete.load(std::memory_order_acquire);
}

CORE_API void SetEngineStartupModuleLoadingComplete()
{
	if (ensure(!GEngineStartupModuleLoadingComplete.load(std::memory_order_relaxed)))
	{
		GEngineStartupModuleLoadingComplete.store(true, std::memory_order_release);
		SCOPED_BOOT_TIMING("OnAllModuleLoadingPhasesComplete.Broadcast");
		FCoreDelegates::OnAllModuleLoadingPhasesComplete.Broadcast();
	}
}

// This should be left non static to allow *edge* cases only in Core to extern and set this.
bool GShouldRequestExit = false;

void CORE_API BeginExitIfRequested()
{
	if (UNLIKELY(GShouldRequestExit))
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		GIsRequestingExit = true;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
}

void CORE_API RequestEngineExit(const TCHAR* ReasonString)
{
	ensureMsgf(ReasonString && FCString::Strlen(ReasonString) > 4, TEXT("RequestEngineExit must be given a valid reason (reason \"%s\""), ReasonString);

	TRACE_BOOKMARK(TEXT("Engine exit requested (reason: %s)"), ReasonString);

	FGenericCrashContext::SetEngineExit(true);

#if UE_SET_REQUEST_EXIT_ON_TICK_ONLY
	UE_LOGF(LogCore, Log, "Engine exit requested (reason: %ls%ls)", ReasonString, GShouldRequestExit ? TEXT("; note: exit was already requested") : TEXT(""));
	GShouldRequestExit = true;
#else
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_LOGF(LogCore, Log, "Engine exit requested (reason: %ls%ls)", ReasonString, GIsRequestingExit ? TEXT("; note: exit was already requested") : TEXT(""));
	GIsRequestingExit = true;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // UE_SET_REQUEST_EXIT_ON_TICK_ONLY
}

void CORE_API RequestEngineExit(const FString& ReasonString)
{
	RequestEngineExit(*ReasonString);
}

/** Exec handler for game debugging tool, allowing commands like "editactor", ...							*/
FExec*					GDebugToolExec					= NULL;
/** Whether we're currently in the async loading codepath or not											*/
static bool IsAsyncLoadingCoreInternal()
{
	// No Async loading in Core
	return false;
}
bool (*IsAsyncLoading)() = &IsAsyncLoadingCoreInternal;
void (*SuspendAsyncLoading)() = &appNoop;
void (*ResumeAsyncLoading)() = &appNoop;
bool (*IsAsyncLoadingSuspended)() = &IsAsyncLoadingCoreInternal;
bool (*IsAsyncLoadingMultithreaded)() = &IsAsyncLoadingCoreInternal;
void (*SuspendTextureStreamingRenderTasks)() = &appNoop;
void (*ResumeTextureStreamingRenderTasks)() = &appNoop;

static ELoaderType LoaderNotInitialized()
{
	return ELoaderType::NotInitialized;
}
ELoaderType (*GetLoaderType)() = &LoaderNotInitialized;

const TCHAR* LexToString(ELoaderType Type)
{
	switch (Type)
	{
	case ELoaderType::NotInitialized:
		return TEXT("NotInitialized");
	case ELoaderType::LegacyLoader:
		return TEXT("LegacyLoader");
	case ELoaderType::EditorPackageLoader:
		return TEXT("EditorPackageLoader");
	case ELoaderType::ZenLoader:
		return TEXT("ZenLoader");
	default:
		check(false);
		return TEXT("");
	}
}

/** Whether the cooker is currently loading a package or not												*/
bool					GIsCookerLoadingPackage			= false;
/**
 * Whether GWorld should point to the play in editor world.
 * @warning There are times when GIsPlayInEditorWorld == true and GWorld == nullptr! Notably during LoadMap 
 * during PIE, which can happen when PIE initiates a Browse to another level.
 */
bool					GIsPlayInEditorWorld			= false;
/** true if the runtime needs textures to be powers of two													*/
bool					GPlatformNeedsPowerOfTwoTextures= false;
/** Time at which FPlatformTime::Seconds() was first initialized (before main)								*/
double					GStartTime						= FPlatformTime::InitTiming();
/** System time at engine init.																				*/
FString					GSystemStartTime;
/** Whether we are still in the initial loading proces.														*/
bool					GIsInitialLoad					= true;
/* Whether we are using the event driven loader */
bool					GEventDrivenLoaderEnabled		= false;

/** Steadily increasing frame counter.																		*/
uint64					GFrameCounter					= 0;
uint64					GFrameCounterRenderThread		= 0;

uint64					GLastGCFrame					= 0;
/** The time input was sampled, in cycles.																	*/
uint64					GInputTime						= 0;
/** Incremented once per frame before the scene is being rendered. In split screen mode this is incremented once for all views (not for each view). */
uint32					GFrameNumber					= 1;
/** NEED TO RENAME, for RT version of GFrameTime use View.ViewFamily->FrameNumber or pass down from RT from GFrameTime). */
uint32					GFrameNumberRenderThread		= 1;
/** Whether we are the first instance of the game running.													*/
PRAGMA_DISABLE_DEPRECATION_WARNINGS
bool					GIsFirstInstance				= true;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
/** Threshold for a frame to be considered a hitch (in milliseconds). */
float GHitchThresholdMS = 60.0f;
/** Size to break up data into when saving compressed data													*/
int32					GSavingCompressionChunkSize		= SAVING_COMPRESSION_CHUNK_SIZE;
/** Thread ID of the main/game thread																		*/
uint32					GGameThreadId					= 0;
uint32					GRenderThreadId					= 0;
uint32					GSlateLoadingThreadId			= 0;
uint32					GAudioThreadId					= 0;
/** Has GGameThreadId been set yet?																			*/
bool					GIsGameThreadIdInitialized		= false;

/** Helper function to flush resource streaming.															*/
void					(*GFlushStreamingFunc)(void)	  = &appNoop;
/** Whether to emit begin/ end draw events.																	*/
bool					GEmitDrawEvents					= false;
/** Determines what kind of trace should occur, NAME_None for none.											*/
FLazyName				GCurrentTraceName;
/** How to print the time in log output																		*/
TSAN_ATOMIC(ELogTimes::Type) GPrintLogTimes             = ELogTimes::None;
/** How to print the category in log output.																*/
TSAN_ATOMIC(bool)		GPrintLogCategory				= true;
/** How to print the verbosity in log output.																*/
TSAN_ATOMIC(bool)		GPrintLogVerbosity				= true;

#if USE_HITCH_DETECTION
TSAN_ATOMIC(bool)				GHitchDetected(false);
#endif

/** Whether stats should emit named events for e.g. PIX.													*/
int32					GCycleStatsShouldEmitNamedEvents = 0;

/** Whether verbose stats should be also generate external profiler named events.
* Thread sleep/wait stats or extremely high frequency cycle counting stats are disabled by default.
* Has no effect if GCycleStatsShouldEmitNamedEvents is 0.
*/
bool					GShouldEmitVerboseNamedEvents = false;

/** Disables some warnings and minor features that would interrupt a demo presentation						*/
bool					GIsDemoMode						= false;
/** Whether or not a unit test is currently being run														*/
bool					GIsAutomationTesting					= false;
/** Whether or not messages are being pumped outside of the main loop										*/
bool					GPumpingMessagesOutsideOfMainLoop = false;
/** Whether or not messages are being pumped */
bool					GPumpingMessages = false;

/** Enables various editor and HMD hacks that allow the experimental VR editor feature to work, perhaps at the expense of other systems */
bool					GEnableVREditorHacks = false;

bool CORE_API			GIsGPUCrashed = false;

#if !UE_BUILD_SHIPPING

/** Whether we should ignore the attached debugger. */
CORE_API bool			GIgnoreDebugger = false;

#endif // #if !UE_BUILD_SHIPPING

bool GetEmitDrawEvents()
{
	return GEmitDrawEvents;
}

void CORE_API SetEmitDrawEvents(bool EmitDrawEvents)
{
	GEmitDrawEvents = EmitDrawEvents;
}

void ToggleGDebugPUCrashedFlag(const TArray<FString>& Args)
{
	GIsGPUCrashed = !GIsGPUCrashed;
	UE_LOGF(LogCore, Log, "Gpu crashed flag forcibly set to: %i", GIsGPUCrashed ? 1 : 0);
}

static struct FBootTimingStart
{
	double FirstTime;

	FBootTimingStart()
		: FirstTime(FPlatformTime::Seconds())
	{
	}
} GBootTimingStart;

FEngineTrackedActivityScope::FEngineTrackedActivityScope(const TCHAR* Fmt, ...)
{
	va_list Args;
	va_start(Args, Fmt);
	TCHAR Str[4096];
	FCString::GetVarArgs(Str, UE_ARRAY_COUNT(Str), Fmt, Args);
	FTrackedActivity::GetEngineActivity().Push(Str);
	va_end(Args);
}

FEngineTrackedActivityScope::FEngineTrackedActivityScope(const FString& Text)
{
	FTrackedActivity::GetEngineActivity().Push(*Text);
}

FEngineTrackedActivityScope::~FEngineTrackedActivityScope()
{
	FTrackedActivity::GetEngineActivity().Pop();
}

static bool GBootTimingCompleted = false;

#define USE_BOOT_PROFILING 0
#define CSV_BOOT_PROFILING (CSV_PROFILER_STATS && !UE_BUILD_SHIPPING)

#if !USE_BOOT_PROFILING
FScopedBootTiming::FScopedBootTiming(const ANSICHAR *InMessage)
{
	UE_LOGFMT(LogInit, Verbose, "{Message}", InMessage);
#if CSV_BOOT_PROFILING
	if (!GBootTimingCompleted && IsInGameThread() && FCsvProfiler::IsCapturing())
	{
		Message = InMessage;
		if (Message.Len())
		{
			CSV_REGION_BEGIN(EngineInit, *Message);
		}
	}
#endif
}

FScopedBootTiming::FScopedBootTiming(const ANSICHAR *InMessage, FName Suffix)
{
	UE_LOGFMT(LogInit, Verbose, "{Message}: {Suffix}", InMessage, Suffix);
#if CSV_BOOT_PROFILING
	if (!GBootTimingCompleted && IsInGameThread() && FCsvProfiler::IsCapturing())
	{
		Message = FString(InMessage) + Suffix.ToString();
		if (Message.Len())
		{
			CSV_REGION_BEGIN(EngineInit, *Message);
		}
	}
#endif
}

FScopedBootTiming::~FScopedBootTiming()
{
#if CSV_BOOT_PROFILING
	if (!Message.IsEmpty())
	{
		CSV_REGION_END(EngineInit, *Message);
	}
#endif

}
void BootTimingPoint(const ANSICHAR *Message)
{
	TRACE_BOOKMARK(TEXT("%s"), *FString(Message));
}
void DumpBootTiming()
{
	GBootTimingCompleted = true;
}
#else
static TArray<FString> GAllBootTiming;
static int32 GBootScopeDepth = 0;

static void DumpBootTimingString(const TCHAR* Message)
{
	if (0)
	{
		UE_LOGF(LogCore, Display, "%ls", Message);
	}
	else
	{
		// Some platforms add an implicit \n if it isn't there, others don't
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("%s\n"), Message);
	}
}

void DumpBootTiming()
{
	GBootTimingCompleted = true;
	DumpBootTimingString(TEXT("************* Boot timing:"));
	for (auto& Item : GAllBootTiming)
	{
		DumpBootTimingString(*Item);
	}
	GAllBootTiming.Empty();
	DumpBootTimingString(TEXT("************* Boot timing end"));
}

static void BootTimingPoint(const TCHAR *Message, const TCHAR *Prefix = nullptr, int32 Depth = 0, double TookTime = 0.0)
{
	TRACE_BOOKMARK(TEXT("%s"), Message);

	static double LastTime = 0.0;
	static TArray<FString> MessageStack;
	static FString LastGapMessage;
	double Now = FPlatformTime::Seconds();

	FString Result;
	FString GapTime;
	FString LastMessage;

	if (MessageStack.Num())
	{
		LastMessage = MessageStack.Last();
	}

	if (!Prefix || FCString::Strcmp(Prefix, TEXT("}")) || LastMessage != FString(Message))
	{
		if (LastTime != 0.0 && float(Now - LastTime) >= 0.005f)
		{
			GapTime = FString::Printf(TEXT("              %7.3fs **Gap**"), float(Now - LastTime));
			GAllBootTiming.Add(GapTime);
			DumpBootTimingString(*FString::Printf(TEXT("[BT]******** %s"), *GapTime));
			LastGapMessage = LastMessage;
		}
	}
	LastTime = Now;

	if (Prefix)
	{
		if (FCString::Strcmp(Prefix, TEXT("}")) == 0)
		{
			if (LastMessage == Message)
			{
				MessageStack.Pop();
				if (LastGapMessage == Message)
				{
					LastGapMessage.Reset();
				}
			}
		}
		else if (FCString::Strcmp(Prefix, TEXT("{")) == 0)
		{
			MessageStack.Add(Message);
		}

		if (TookTime != 0.0)
		{
			Result = FString::Printf(TEXT("%7.3fs took %7.3fs %s   %1s %s"), float(Now - GBootTimingStart.FirstTime), float(TookTime), FCString::Spc(Depth * 2), Prefix, Message);
		}
		else
		{
			Result = FString::Printf(TEXT("%7.3fs               %s   %1s %s"), float(Now - GBootTimingStart.FirstTime), FCString::Spc(Depth * 2), Prefix, Message);
		}
	}
	else
	{
		if (TookTime != 0.0)
		{
			Result = FString::Printf(TEXT("%7.3fs took %7.3fs : %s"), float(Now - GBootTimingStart.FirstTime), float(TookTime), Message);
		}
		else
		{
			Result = FString::Printf(TEXT("%7.3fs : %s"), float(Now - GBootTimingStart.FirstTime), Message);
		}
	}
	bool bKeep = true;
	if (Prefix && TookTime > 0.0 && GAllBootTiming.Num())
	{
		// Don't suppress the first 0 time block if there was a gap right before
		if (MessageStack.Num() != 0 && MessageStack.Last() == LastGapMessage)
		{
			LastGapMessage.Reset();
		}
		else if (TookTime < 0.001)
		{
			// Remove a paired {} if time was too small
			if (GAllBootTiming.Last().Contains(Message))
			{
				GAllBootTiming.Pop();
				bKeep = false;
			}
		}
	}
	if (bKeep)
	{
		GAllBootTiming.Add(Result);
	}
	DumpBootTimingString(*FString::Printf(TEXT("[BT]******** %s"), *Result));
}

void BootTimingPoint(const ANSICHAR *Message)
{
	if (GBootTimingCompleted || !IsInGameThread())
	{
		return;
	}
	BootTimingPoint(*FString(Message));
}

FScopedBootTiming::FScopedBootTiming(const ANSICHAR *InMessage)
{
	if (GBootTimingCompleted || !IsInGameThread())
	{
		return;
	}
	Message = InMessage;
	StartTime = FPlatformTime::Seconds();
	BootTimingPoint(*Message, TEXT("{"), GBootScopeDepth);
#if CSV_BOOT_PROFILING
	CSV_REGION_BEGIN(EngineInit, *Message);
#endif
	GBootScopeDepth++;
}

FScopedBootTiming::FScopedBootTiming(const ANSICHAR *InMessage, FName Suffix)
{
	if (GBootTimingCompleted || !IsInGameThread())
	{
		return;
	}
	Message = FString(InMessage) + Suffix.ToString();
	StartTime = FPlatformTime::Seconds();
	BootTimingPoint(*Message, TEXT("{"), GBootScopeDepth);
#if CSV_BOOT_PROFILING
	CSV_REGION_BEGIN(EngineInit, *Message);
#endif
	GBootScopeDepth++;
}

FScopedBootTiming::~FScopedBootTiming()
{
	if (Message.Len())
	{
		GBootScopeDepth--;
		BootTimingPoint(*Message, TEXT("}"), GBootScopeDepth, FPlatformTime::Seconds() - StartTime);
#if CSV_BOOT_PROFILING
		CSV_REGION_END(EngineInit, *Message);
#endif
	}
}

#endif

FAutoConsoleCommand ToggleDebugGPUCrashedCmd(
	TEXT("c.ToggleGPUCrashedFlagDbg"),
	TEXT("Forcibly toggles the 'GPU Crashed' flag for testing crash analytics."),
	FConsoleCommandWithArgsDelegate::CreateStatic(&ToggleGDebugPUCrashedFlag),
	ECVF_Cheat);


DEFINE_STAT(STAT_AudioMemory);
DEFINE_STAT(STAT_TextureMemory);
DEFINE_STAT(STAT_MemoryPhysXTotalAllocationSize);
DEFINE_STAT(STAT_MemoryICUTotalAllocationSize);
DEFINE_STAT(STAT_MemoryICUDataFileAllocationSize);
DEFINE_STAT(STAT_PrecomputedVisibilityMemory);
DEFINE_STAT(STAT_SkeletalMeshVertexMemory);
DEFINE_STAT(STAT_SkeletalMeshIndexMemory);
DEFINE_STAT(STAT_SkeletalMeshMotionBlurSkinningMemory);
DEFINE_STAT(STAT_VertexShaderMemory);
DEFINE_STAT(STAT_PixelShaderMemory);
DEFINE_STAT(STAT_NavigationMemory);

DEFINE_STAT(STAT_ReflectionCaptureTextureMemory);
DEFINE_STAT(STAT_ReflectionCaptureMemory);

/** Threading stats objects */

DEFINE_STAT(STAT_RenderingIdleTime_WaitingForGPUQuery);
DEFINE_STAT(STAT_RenderingIdleTime_WaitingForGPUPresent);
DEFINE_STAT(STAT_RenderingIdleTime_RenderThreadSleepTime);

DEFINE_STAT(STAT_RenderingIdleTime);
DEFINE_STAT(STAT_RenderingBusyTime);
DEFINE_STAT(STAT_GameIdleTime);
DEFINE_STAT(STAT_GameTickWaitTime);
DEFINE_STAT(STAT_GameTickWantedWaitTime);
DEFINE_STAT(STAT_GameTickAdditionalWaitTime);

DEFINE_STAT(STAT_TaskGraph_OtherTasks);
DEFINE_STAT(STAT_TaskGraph_OtherStalls);

DEFINE_STAT(STAT_TaskGraph_RenderStalls);

DEFINE_STAT(STAT_TaskGraph_GameTasks);
DEFINE_STAT(STAT_TaskGraph_GameStalls);

DEFINE_STAT(STAT_CPUTimePct);
DEFINE_STAT(STAT_CPUTimePctRelative);

DEFINE_LOG_CATEGORY(LogHAL);
DEFINE_LOG_CATEGORY(LogSerialization);
DEFINE_LOG_CATEGORY(LogContentComparisonCommandlet);
DEFINE_LOG_CATEGORY(LogNetPackageMap);
DEFINE_LOG_CATEGORY(LogNetSerialization);
DEFINE_LOG_CATEGORY(LogMemory);
DEFINE_LOG_CATEGORY(LogProfilingDebugging);
DEFINE_LOG_CATEGORY(LogTemp);
DEFINE_LOG_CATEGORY(LogVirtualization);

// need another layer of macro to help using a define in a define
#define DEFINE_LOG_CATEGORY_HELPER(A) DEFINE_LOG_CATEGORY(A)
#ifdef PLATFORM_GLOBAL_LOG_CATEGORY
	DEFINE_LOG_CATEGORY_HELPER(PLATFORM_GLOBAL_LOG_CATEGORY);
#endif
#ifdef PLATFORM_GLOBAL_LOG_CATEGORY_ALT
	DEFINE_LOG_CATEGORY_HELPER(PLATFORM_GLOBAL_LOG_CATEGORY_ALT);
#endif

thread_local bool PRIVATE_GIsDuplicatingClassForReinstancing = false;

FIsDuplicatingClassForReinstancing& FIsDuplicatingClassForReinstancing::operator= (bool bOther)
{
	PRIVATE_GIsDuplicatingClassForReinstancing = bOther;
	return *this;
}

FIsDuplicatingClassForReinstancing::operator bool() const
{
	return PRIVATE_GIsDuplicatingClassForReinstancing;
}

namespace PlayInEditorIDImpl
{
	int32 PRIVATE_GPlayInEditorID_GameThread = -1;
	// We need to differentiate between the game-thread being identified as the loading thread during postload 
	// and the actual loading thread to avoid scopes to overlap and race between both threads.
	int32 PRIVATE_GPlayInEditorID_GameThreadAsLoadingThread = -2;
	int32 PRIVATE_GPlayInEditorID_ActualLoadingThread = -2;

	static int32* GetPointer()
	{
		if (IsInAsyncLoadingThread())
		{
			if (IsInGameThread())
			{
				return &PRIVATE_GPlayInEditorID_GameThreadAsLoadingThread;
			}
			else
			{
				return &PRIVATE_GPlayInEditorID_ActualLoadingThread;
			}
		}
		else if (IsInGameThread())
		{
			return &PRIVATE_GPlayInEditorID_GameThread;
		}

		return nullptr;
	}
};

int32 PRIVATE_GetGPlayInEditorID()
{
	if (int32* Pointer = PlayInEditorIDImpl::GetPointer())
	{
		return *Pointer;
	}
	else
	{
		// GPlayInEditorID doesn't have a value on worker threads. If it's needed, it should be captured in the task context.
		return -1;
	}
}

void PRIVATE_SetGPlayInEditorID(int32 InValue)
{
	if (int32* Pointer = PlayInEditorIDImpl::GetPointer())
	{
		*Pointer = InValue;
	}
	else
	{
		// There is no value on worker thread... so just do nothing here. -1 is always returned anyway.
	}
}

namespace UE::Core::Private
{
	FPlayInEditorLoadingScope::FPlayInEditorLoadingScope(int32 PlayInEditorID)
		: OldValue(PRIVATE_GetGPlayInEditorID())
	{
		PRIVATE_SetGPlayInEditorID(PlayInEditorID);
	}

	FPlayInEditorLoadingScope::~FPlayInEditorLoadingScope()
	{
		PRIVATE_SetGPlayInEditorID(OldValue);
	}
}

namespace UE
{
	int32 GetPlayInEditorID()
	{
		int32 Value = PRIVATE_GetGPlayInEditorID();
		checkf(Value != -2, TEXT("GPlayInEditorID has not been properly forwarded by the loading-thread."));
		return Value;
	}
	void SetPlayInEditorID(int32 InPlayInEditorID)
	{
		PRIVATE_SetGPlayInEditorID(InPlayInEditorID);
	}
}

bool PRIVATE_GIsEditorLoadingPackage = false;

namespace UE
{
	bool GetIsEditorLoadingPackage()
	{
		if (IsInGameThread())
		{
			if (GIsEditor && IsInAsyncLoadingThread())
			{
				ensureMsgf(PRIVATE_GIsEditorLoadingPackage, TEXT("GIsEditorLoadingPackage should always be true during editor loading."));
			}
			return PRIVATE_GIsEditorLoadingPackage;
		}
		else if (IsInAsyncLoadingThread())
		{
			// Directly depends on GIsEditor on real async-loading thread
			return GIsEditor;
		}
		else
		{
			ensureMsgf(false, TEXT("Querying GIsEditorLoadingPackage from any thread is not safe, you should capture the value instead if needed."));
			return false;
		}
	}
	void SetIsEditorLoadingPackage(bool InValue)
	{
		if (IsInGameThread())
		{
			if (GIsEditor && !InValue && IsInAsyncLoadingThread())
			{
				ensureMsgf(false, TEXT("Overriding GIsEditorLoadingPackage to false during editor loading is prohibited."));
			}
			else
			{
				PRIVATE_GIsEditorLoadingPackage = InValue;
			}
		}
		else
		{
			ensureMsgf(false, TEXT("Overriding GIsEditorLoadingPackage only makes sense for the game thread."));
		}
	}
}

#undef LOCTEXT_NAMESPACE

bool IsRunningCookOnTheFly()
{
#if WITH_COTF
	static struct FCookOnTheFlyCommandline
	{
		bool bParsed;
		FCookOnTheFlyCommandline(const TCHAR* CmdLine)
		{
			FString Host;
			bParsed = FParse::Value(CmdLine, TEXT("-FileHostIP="), Host);
		}
	} CookOnTheFlyCommandline(FCommandLine::Get());
	
	return CookOnTheFlyCommandline.bParsed;
#else
	return false;
#endif
}

CORE_API FString GetCommandletNameFromCmdline()
{
	FString CommandletName;

#if WITH_ENGINE
	FParse::Value(FCommandLine::Get(), TEXT("-run="), CommandletName);
#endif

	return CommandletName;
}

namespace UE
{

int32 GetMultiprocessId()
{
#if WITH_EDITOR
	return UE::Private::GMultiprocessId;
#else
	return 0;
#endif
}

int GIsUsingZenPakFileStreaming = -1;

bool IsUsingZenPakFileStreaming()
{
	if (GIsUsingZenPakFileStreaming == -1)
	{
		if (FParse::Param(FCommandLine::Get(), TEXT("ZenPak")))
		{
			GIsUsingZenPakFileStreaming = 1;
		}
		else
		{
			GIsUsingZenPakFileStreaming = 0;
		}
	}

	return GIsUsingZenPakFileStreaming == 1;
}

}

bool IsThisNotNull(const void* This, const ANSICHAR* Function)
{
	if (This)
	{
		return true;
	}

	UE_LOGF(LogCore, Warning, "'%s' was called with a null 'this' pointer - this is no longer supported.", Function);
	return false;
}

namespace UE::Private
{

void SetMultiprocessId(int32 MultiprocessId)
{
#if WITH_EDITOR
	UE::Private::GMultiprocessId = MultiprocessId;
#endif
}

FScopeDisableRunningCookCommandlet::FScopeDisableRunningCookCommandlet()
{
#if WITH_EDITOR
	bPrevious = PRIVATE_GIsRunningCookCommandlet;
	PRIVATE_GIsRunningCookCommandlet = false;
#endif
}

FScopeDisableRunningCookCommandlet::~FScopeDisableRunningCookCommandlet()
{
#if WITH_EDITOR
	PRIVATE_GIsRunningCookCommandlet = bPrevious;
#endif
}
}

bool(*IsGCLockedFuncPtr)() = nullptr;

namespace UE
{
bool IsGCLocked()
{
	return IsGCLockedFuncPtr && IsGCLockedFuncPtr();
}

}
namespace UE::Private
{
void SetIsGCLockedFunction(bool(*InIsGCLockedFuncPtr)())
{
	IsGCLockedFuncPtr = InIsGCLockedFuncPtr;
}

}

