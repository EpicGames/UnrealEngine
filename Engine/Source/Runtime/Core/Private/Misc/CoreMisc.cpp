// Copyright Epic Games, Inc. All Rights Reserved.

// Core includes.
#include "Misc/CoreMisc.h"
#include "Misc/Parse.h"
#include "Misc/CommandLine.h"
#include "Containers/Ticker.h"
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformTime.h"
#include "Misc/App.h"
#include "Misc/LazySingleton.h"
#include "Misc/OutputDeviceError.h"
#include "Misc/ScopeLock.h"
#include "Misc/ScopeRWLock.h"
#include "CoreGlobals.h"
#include "Templates/RefCounting.h"

/** For FConfigFile in appInit							*/
#include "Misc/ConfigCacheIni.h"

#include "Modules/ModuleManager.h"
#include "DerivedDataCacheModule.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Interfaces/ITargetPlatform.h"

DEFINE_LOG_CATEGORY(LogSHA);
DEFINE_LOG_CATEGORY(LogStats);
DEFINE_LOG_CATEGORY(LogStreaming);
DEFINE_LOG_CATEGORY(LogInit);
DEFINE_LOG_CATEGORY(LogExit);
DEFINE_LOG_CATEGORY(LogExec);
DEFINE_LOG_CATEGORY(LogScript);
DEFINE_LOG_CATEGORY(LogLocalization);
DEFINE_LOG_CATEGORY(LogLongPackageNames);
DEFINE_LOG_CATEGORY(LogProcess);
DEFINE_LOG_CATEGORY(LogLoad);
DEFINE_LOG_CATEGORY(LogCore);


/*-----------------------------------------------------------------------------
	FSelfRegisteringExec implementation.
-----------------------------------------------------------------------------*/

using FSelfRegisteredExecArray = TArray<FSelfRegisteringExec*, TInlineAllocator<8>>;

// Lazy because pthread implementation doesn't like static initialization.
FTransactionallySafeCriticalSection* GetExecRegistryLock()
{
	// Note: Using FCriticalSection still allows calls to addition or removal
	// to/from Execs and FSelfRegisteringExec::StaticExec on the same thread
	static FTransactionallySafeCriticalSection ExecRegistryLock;
	return &ExecRegistryLock;
}

FSelfRegisteredExecArray& GetExecRegistry()
{
	static FSelfRegisteredExecArray Execs;
	return Execs;
}

/** Constructor, registering this instance. */
FSelfRegisteringExec::FSelfRegisteringExec()
{
	UE::TScopeLock ScopeLock(*GetExecRegistryLock());
	GetExecRegistry().Add( this );
}

/** Destructor, unregistering this instance. */
FSelfRegisteringExec::~FSelfRegisteringExec()
{
	UE::TScopeLock ScopeLock(*GetExecRegistryLock());
	verify(GetExecRegistry().Remove( this ) == 1 );
}

bool FSelfRegisteringExec::StaticExec( UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar )
{
	UE::TScopeLock ScopeLock(*GetExecRegistryLock());
	for (FSelfRegisteringExec* Exe : GetExecRegistry())
	{
		if (Exe->Exec( InWorld, Cmd,Ar ))
		{
			return true;
		}
	}

	return false;
}

FStaticSelfRegisteringExec::FStaticSelfRegisteringExec(bool (*InStaticExecFunc)(UWorld* Inworld, const TCHAR* Cmd,FOutputDevice& Ar))
:	StaticExecFunc(InStaticExecFunc)
{}

#if UE_ALLOW_EXEC_COMMANDS
bool FStaticSelfRegisteringExec::Exec( UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar )
{
	return (*StaticExecFunc)( InWorld, Cmd,Ar);
}
#endif // UE_ALLOW_EXEC_COMMANDS

FStaticSelfRegisteringExec_Dev::FStaticSelfRegisteringExec_Dev(bool (*InStaticExecFunc)(UWorld* Inworld, const TCHAR* Cmd,FOutputDevice& Ar))
	: StaticExecFunc(InStaticExecFunc)
{}

bool FStaticSelfRegisteringExec_Dev::Exec_Dev(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	return (*StaticExecFunc)(InWorld, Cmd, Ar);
}

FStaticSelfRegisteringExec_Editor::FStaticSelfRegisteringExec_Editor(bool (*InStaticExecFunc)(UWorld* Inworld, const TCHAR* Cmd, FOutputDevice& Ar))
	: StaticExecFunc(InStaticExecFunc)
{}

bool FStaticSelfRegisteringExec_Editor::Exec_Editor(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	return (*StaticExecFunc)(InWorld, Cmd, Ar);
}

// Remove old crash contexts


/*-----------------------------------------------------------------------------
	Module singletons.
-----------------------------------------------------------------------------*/

FDerivedDataCacheInterface* GetDerivedDataCache()
{
	static FDerivedDataCacheInterface* const* Cache;
	static bool bInitialized = false;
	if (!bInitialized)
	{
		if (!FPlatformProperties::RequiresCookedData())
		{
			check(IsInGameThread());
			bInitialized = true;
			if (IDerivedDataCacheModule* Module = FModuleManager::LoadModulePtr<IDerivedDataCacheModule>("DerivedDataCache"))
			{
				Cache = Module->CreateOrGetCache();
			}
		}
	}
	return Cache ? *Cache : nullptr;
}

FDerivedDataCacheInterface& GetDerivedDataCacheRef()
{
	FDerivedDataCacheInterface* Cache = GetDerivedDataCache();
	if (!Cache)
	{
		UE_LOGF(LogInit, Fatal, "Derived Data Cache was requested, but not available.");
		CA_ASSUME(Cache); // Suppress static analysis warning in unreachable code (fatal error)
	}
	return *Cache;
}

FDerivedDataCacheInterface* TryGetDerivedDataCache()
{
	if (IDerivedDataCacheModule* Module = FModuleManager::GetModulePtr<IDerivedDataCacheModule>("DerivedDataCache"))
	{
		if (FDerivedDataCacheInterface* const* Cache = Module->GetCache())
		{
			return *Cache;
		}
	}
	return nullptr;
}

class ITargetPlatformManagerModule* GetTargetPlatformManager(bool bFailOnInitErrors)
{
	static class ITargetPlatformManagerModule* SingletonInterface = NULL;
	if (!FPlatformProperties::RequiresCookedData())
	{
		static bool bInitialized = false;
		if (!bInitialized)
		{
			check(IsInGameThread());
			bInitialized = true;
			SingletonInterface = ITargetPlatformManagerModule::Get();

			FString InitErrors;
			if (bFailOnInitErrors && SingletonInterface && SingletonInterface->HasInitErrors(&InitErrors))
			{
				GError->Log(*InitErrors);
			}
		}
	}
	return SingletonInterface;
}

class ITargetPlatformManagerModule& GetTargetPlatformManagerRef()
{
	class ITargetPlatformManagerModule* SingletonInterface = GetTargetPlatformManager();
	if (!SingletonInterface)
	{
		UE_LOGF(LogInit, Fatal, "Target platform manager was requested, but not available.");
		CA_ASSUME( SingletonInterface != NULL );	// Suppress static analysis warning in unreachable code (fatal error)
	}
	return *SingletonInterface;
}

bool WillNeedAudioVisualData()
{
	class ITargetPlatformManagerModule* SingletonInterface = GetTargetPlatformManager();
#if WITH_ENGINE
	// quick check to see if we are targeting non-running platforms
	if (SingletonInterface && SingletonInterface->RestrictFormatsToRuntimeOnly() == false)
	{
		for (ITargetPlatform* Platform : SingletonInterface->GetActiveTargetPlatforms())
		{
			if (Platform->AllowAudioVisualData())
			{
				return true;
			}
		}

		// if nothing in the loop above returned true, then we don't need AV data
		return false;
	}
#endif

	// default to needing AV data because some commandlets may need the data for processing - otherwise we could use "FApp::CanEverRender() || FApp::CanEverRenderAudio()"
	return true;
}

/*----------------------------------------------------------------------------
	Runtime functions.
----------------------------------------------------------------------------*/

FQueryIsRunningServer GIsServerDelegate;

bool IsServerForOnlineSubsystems(FName WorldContextHandle)
{
	if (GIsServerDelegate.IsBound())
	{
		return GIsServerDelegate.Execute(WorldContextHandle);
	}
	else
	{
		return IsRunningDedicatedServer();
	}
}

void SetIsServerForOnlineSubsystemsDelegate(FQueryIsRunningServer NewDelegate)
{
	GIsServerDelegate = NewDelegate;
}

#if UE_EDITOR

/** Checks the command line for the presence of switches to indicate running as "dedicated server only" */
int32 CORE_API StaticDedicatedServerCheck()
{
	static int32 HasServerSwitch = -1;
	if (HasServerSwitch == -1)
	{
		const FString CmdLine = FString(FCommandLine::Get()).TrimStart();
		const TCHAR* TCmdLine = *CmdLine;

		TArray<FString> Tokens;
		TArray<FString> Switches;
		FCommandLine::Parse(TCmdLine, Tokens, Switches);

		HasServerSwitch = (Switches.Contains(TEXT("SERVER")) || Switches.Contains(TEXT("RUN=SERVER"))) ? 1 : 0;
	}
	return HasServerSwitch;
}

/** Checks the command line for the presence of switches to indicate running as "game only" */
int32 CORE_API StaticGameCheck()
{
	static int32 HasGameSwitch = -1;
	if (HasGameSwitch == -1)
	{
		const FString CmdLine = FString(FCommandLine::Get()).TrimStart();
		const TCHAR* TCmdLine = *CmdLine;

		TArray<FString> Tokens;
		TArray<FString> Switches;
		FCommandLine::Parse(TCmdLine, Tokens, Switches);

		if (Switches.Contains(TEXT("GAME")))
		{
			HasGameSwitch = 1;
		}
		else
		{
			HasGameSwitch = 0;
		}
	}
	return HasGameSwitch;
}

/** Checks the command line for the presence of switches to indicate running as "client only" */
int32 CORE_API StaticClientOnlyCheck()
{
	static int32 HasClientSwitch = -1;
	if (HasClientSwitch == -1)
	{
		const FString CmdLine = FString(FCommandLine::Get()).TrimStart();
		const TCHAR* TCmdLine = *CmdLine;

		TArray<FString> Tokens;
		TArray<FString> Switches;
		FCommandLine::Parse(TCmdLine, Tokens, Switches);

		HasClientSwitch = (StaticGameCheck() && Switches.Contains(TEXT("ClientOnly"))) ? 1 : 0;
	}
	return HasClientSwitch;
}

#endif //UE_EDITOR

void FUrlConfig::Init()
{
	DefaultProtocol = GConfig->GetStr( TEXT("URL"), TEXT("Protocol"), GEngineIni );
	DefaultName = GConfig->GetStr( TEXT("URL"), TEXT("Name"), GEngineIni );
	// strip off any file extensions from map names
	DefaultHost = GConfig->GetStr( TEXT("URL"), TEXT("Host"), GEngineIni );
	DefaultPortal = GConfig->GetStr( TEXT("URL"), TEXT("Portal"), GEngineIni );
	DefaultSaveExt = GConfig->GetStr( TEXT("URL"), TEXT("SaveExt"), GEngineIni );
	
	FString Port;
	// Allow the command line to override the default port
	if (FParse::Value(FCommandLine::Get(),TEXT("Port="),Port) == false)
	{
		Port = GConfig->GetStr( TEXT("URL"), TEXT("Port"), GEngineIni );
	}
	DefaultPort = FCString::Atoi( *Port );
}

void FUrlConfig::Reset()
{
	DefaultProtocol = TEXT("");
	DefaultName = TEXT("");
	DefaultHost = TEXT("");
	DefaultPortal = TEXT("");
	DefaultSaveExt = TEXT("");
}

bool CORE_API StringHasBadDashes(const TCHAR* Str)
{
	// Detect dashes (0x2013) and report an error if they're found
	while (TCHAR Ch = *Str++)
	{
		if ((UCS2CHAR)Ch == 0x2013)
			return true;
	}

	return false;
}

/*----------------------------------------------------------------------------
FBoolConfigValueHelper
----------------------------------------------------------------------------*/
FBoolConfigValueHelper::FBoolConfigValueHelper(const TCHAR* Section, const TCHAR* Key, const FString& Filename)
	: bValue(false)
{
	GConfig->GetBool(Section, Key, bValue, Filename);
}

/*----------------------------------------------------------------------------
FScriptExceptionHandler
----------------------------------------------------------------------------*/

UE_DEFINE_THREAD_SINGLETON_TLS(FScriptExceptionHandler, CORE_API)

FScriptExceptionHandlerFunc FScriptExceptionHandler::DefaultExceptionHandler = &FScriptExceptionHandler::LoggingExceptionHandler;

FScriptExceptionHandler& FScriptExceptionHandler::Get()
{
	return TThreadSingleton<FScriptExceptionHandler>::Get();
}

void FScriptExceptionHandler::PushExceptionHandler(const FScriptExceptionHandlerFunc& InFunc)
{
	ExceptionHandlerStack.Push(InFunc);
}

void FScriptExceptionHandler::PopExceptionHandler()
{
	check(ExceptionHandlerStack.Num() > 0);
	ExceptionHandlerStack.Pop(EAllowShrinking::No);
}

void FScriptExceptionHandler::HandleException(ELogVerbosity::Type Verbosity, const TCHAR* ExceptionMessage, const TCHAR* StackMessage)
{
	if (ExceptionHandlerStack.Num() > 0)
	{
		ExceptionHandlerStack.Top()(Verbosity, ExceptionMessage, StackMessage);
	}
	else
	{
		DefaultExceptionHandler(Verbosity, ExceptionMessage, StackMessage);
	}
}

void FScriptExceptionHandler::AssertionExceptionHandler(ELogVerbosity::Type Verbosity, const TCHAR* ExceptionMessage, const TCHAR* StackMessage)
{
	// Ensure for errors and warnings, for everything else just log
	if (Verbosity <= ELogVerbosity::Error)
	{
		ensureAlwaysMsgf(false, TEXT("Script Msg: %s\n%s"), ExceptionMessage, StackMessage);
	}
	else
	{
		LoggingExceptionHandler(Verbosity, ExceptionMessage, StackMessage);
	}
}

void FScriptExceptionHandler::LoggingExceptionHandler(ELogVerbosity::Type Verbosity, const TCHAR* ExceptionMessage, const TCHAR* StackMessage)
{
#if !NO_LOGGING
	// Call directly so we can pass verbosity through
	FMsg::Logf_Internal(__FILE__, __LINE__, LogScript.GetCategoryName(), Verbosity, TEXT("Script Msg: %s"), ExceptionMessage);
	if (*StackMessage)
	{
		FMsg::Logf_Internal(__FILE__, __LINE__, LogScript.GetCategoryName(), Verbosity, TEXT("%s"), StackMessage);
	}
#endif
}

/*----------------------------------------------------------------------------
FScopedScriptExceptionHandler
----------------------------------------------------------------------------*/
FScopedScriptExceptionHandler::FScopedScriptExceptionHandler(const FScriptExceptionHandlerFunc& InFunc)
{
	FScriptExceptionHandler::Get().PushExceptionHandler(InFunc);
}

FScopedScriptExceptionHandler::~FScopedScriptExceptionHandler()
{
	FScriptExceptionHandler::Get().PopExceptionHandler();
}

bool GIsRetrievingVTablePtr = false;

void EnsureRetrievingVTablePtrDuringCtor(const TCHAR* CtorSignature)
{
	UE_CLOGF(!GIsRetrievingVTablePtr, LogCore, Fatal, "The %ls constructor is for internal usage only for hot-reload purposes. Please do NOT use it.", CtorSignature);
}

/*----------------------------------------------------------------------------
Boot timing
----------------------------------------------------------------------------*/

#if !UE_BUILD_SHIPPING

void NotifyLoadingStateChanged(bool bState, const TCHAR *Message)
{
	static bool bEnabled = FParse::Param(FCommandLine::Get(), TEXT("TrackBootLoading"));
	if (!bEnabled)
	{
		return;
	}

	static FCriticalSection Crit;
	FScopeLock Lock(&Crit);
	static double GLastTimeForNotifyAsyncLoadingStateHasMaybeChanged = FPlatformTime::Seconds();

	static double TotalActiveTime = 0.0;
	static double TotalInactiveTime = 0.0;
	static int32 LoadCount = 0;
	static int32 RecursiveCount = 0;

	double Now = FPlatformTime::Seconds();
	double Diff = Now - GLastTimeForNotifyAsyncLoadingStateHasMaybeChanged;

	if (bState)
	{
		RecursiveCount++;
		UE_LOGF(LogStreaming, Display, "Loading Interval Starting %ls", Message);
	}
	else
	{
		RecursiveCount--;
		check(RecursiveCount >= 0);
		UE_LOGF(LogStreaming, Display, "Loading Interval Ending   %ls", Message);
	}

	if (RecursiveCount == 1 && bState)
	{
		TotalInactiveTime += Diff;
	}
	else
	{
		TotalActiveTime += Diff;
	}

	if (!RecursiveCount)
	{
		LoadCount++;
		UE_LOGF(LogStreaming, Display, "Loading Interval  %5d loading time intervals   %7.2fs spent loading    %7.2fs spent not loading", LoadCount, TotalActiveTime, TotalInactiveTime);
	}
	GLastTimeForNotifyAsyncLoadingStateHasMaybeChanged = Now;
}

#endif


/*----------------------------------------------------------------------------
NAN Diagnostic Failure
----------------------------------------------------------------------------*/

int32 GEnsureOnNANDiagnostic = false;

#if ENABLE_NAN_DIAGNOSTIC
static FAutoConsoleVariableRef CVarGEnsureOnNANDiagnostic(
	TEXT( "EnsureOnNaNFail" ),
	GEnsureOnNANDiagnostic,
	TEXT( "If set to 1 NaN Diagnostic failures will result in ensures being emitted" )
	);
#endif

#if DO_CHECK
namespace UEAsserts_Private
{
	void VARARGS InternalLogNANDiagnosticMessage(const TCHAR* FormattedMsg, ...)
	{		
		const int32 TempStrSize = 4096;
		TCHAR TempStr[TempStrSize];
		GET_TYPED_VARARGS(TCHAR, TempStr, TempStrSize, TempStrSize - 1, FormattedMsg, FormattedMsg);
		UE_LOGF(LogCore, Error, "%ls", TempStr);
	}
}
#endif

static bool GAutoEnableNamedEventsWhenProfiling = false;
static FAutoConsoleVariableRef GCVarAutoEnableNamedEventsWhenProfiling(
	TEXT("stats.AutoEnableNamedEventsWhenProfiling"),
	GAutoEnableNamedEventsWhenProfiling,
	TEXT("If 1, toggles named events on when a profiler is detected and capturing. Toggles named events off if 0 or when profiling stops."),
	ECVF_Default
);

void FAutoNamedEventsToggler::Update(bool bIsProfiling)
{
	if (bIsProfiling && !bSetNamedEventsEnabled && GAutoEnableNamedEventsWhenProfiling)
	{
		++GCycleStatsShouldEmitNamedEvents;
		bSetNamedEventsEnabled = true;
	}
	else if (!bIsProfiling && bSetNamedEventsEnabled)
	{
		GCycleStatsShouldEmitNamedEvents = FMath::Max(0, GCycleStatsShouldEmitNamedEvents - 1);
		bSetNamedEventsEnabled = false;
	}
}

static FAutoConsoleCommand GSetStatsNamedEventsCmd(
	TEXT("stats.NamedEvents"),
	TEXT("<on/off> Enables or disables the Named Events."),
	FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateLambda([](const TArray<FString>& Args, UWorld*, FOutputDevice& Ar)
	{
		if (Args.Num() > 0)
		{
			bool bShouldEnableNamedEvents = FCString::ToBool(*Args[0]);
			if (bShouldEnableNamedEvents)
			{
				if (GCycleStatsShouldEmitNamedEvents == 0)
				{
					GCycleStatsShouldEmitNamedEvents = 1;
				}
			}
			else
			{
				if (GCycleStatsShouldEmitNamedEvents > 0)
				{
					GCycleStatsShouldEmitNamedEvents = 0;
				}
			}
		}
		else
		{
			Ar.Logf(TEXT("Expected 0/1 argument"));
		}
	})
);

static FAutoConsoleCommand GSetStatsVerboseNamedEventsCmd(
	TEXT("stats.VerboseNamedEvents"),
	TEXT("<on/off> Enables or disables the Verbose Named Events."),
	FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateLambda([](const TArray<FString>& Args, UWorld*, FOutputDevice& Ar)
	{
		if (Args.Num() > 0)
		{
			GShouldEmitVerboseNamedEvents = FCString::ToBool(*Args[0]);

			if (GShouldEmitVerboseNamedEvents && GCycleStatsShouldEmitNamedEvents == 0)
			{
				GCycleStatsShouldEmitNamedEvents = 1;
			}
		}
		else
		{
			Ar.Logf(TEXT("Expected on/off argument"));
		}
	})
);