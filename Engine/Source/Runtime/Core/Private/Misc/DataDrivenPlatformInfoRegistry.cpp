// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/DataDrivenPlatformInfoRegistry.h"
#include "Misc/Paths.h"
#include "HAL/FileManager.h"
#include "Misc/FileHelper.h"
#include "Misc/ConfigCacheIni.h"
#include "Modules/ModuleManager.h"
#include "Misc/CommandLine.h"
#include "Containers/Set.h"


namespace 
{
	TMap<FName, FDataDrivenPlatformInfo> DataDrivenPlatforms;
	TMap<FName, FName> GlobalPlatformNameAliases;
	TArray<FName> AllSortedPlatformNames;
	TArray<const FDataDrivenPlatformInfo*> AllSortedPlatformInfos;
	TArray<FName> SortedPlatformNames;
	TArray<const FDataDrivenPlatformInfo*> SortedPlatformInfos;
#if DDPI_HAS_EXTENDED_PLATFORMINFO_DATA
	TMap<FName, TArray<FPreviewPlatformMenuItem>> ShaderPlatformNameToMenuItem;
	TArray<struct FPreviewPlatformMenuItem> PreviewPlatformMenuItems;
	TSet<FName> PlatformsHiddenFromUI;
#endif
}

static FString GetParentIniFromDDPI(const FString& DDPIPath)
{
	FString IniParent;
	FString IniContents;
	if (FFileHelper::LoadFileToString(IniContents, *DDPIPath))
	{
		FConfigFile IniFile;
		IniFile.ProcessInputFileContents(IniContents, DDPIPath);

		IniFile.GetString(TEXT("DataDrivenPlatformInfo"), TEXT("IniParent"), IniParent);
	}
	return IniParent;
}

static const TArray<FString> GetAllDataDrivenIniFilenames()
{
	TArray<FString> DataDrivenIniFilenames;

	// look for the special files in any config subdirectories
	IFileManager::Get().FindFilesRecursive(DataDrivenIniFilenames, *FPaths::EngineConfigDir(), TEXT("DataDrivenPlatformInfo.ini"), true, false);

	// manually look through the platform directories - we can't use GetExtensionDirs(), since that function uses the results of this function 
	TArray<FString> PlatformDirs;
	IFileManager::Get().FindFiles(PlatformDirs, *FPaths::Combine(FPaths::EngineDir(), TEXT("Platforms"), TEXT("*")), false, true);

	for (const FString& PlatformDir : PlatformDirs)
	{
		FString IniPath = FPaths::Combine(FPaths::EnginePlatformExtensionDir(*PlatformDir), TEXT("Config/DataDrivenPlatformInfo.ini"));
		if (IFileManager::Get().FileExists(*IniPath))
		{
			DataDrivenIniFilenames.Add(IniPath);
		}
	}

	// look for the special files in any project config subdirectories
	TArray<FString> ProjectPlatformDirs;
	IFileManager::Get().FindFiles(ProjectPlatformDirs, *FPaths::Combine(FPaths::ProjectConfigDir(), TEXT("*")), false, true);
	for (const FString& PlatformDir : ProjectPlatformDirs)
	{
		FString IniPath = FPaths::Combine(FPaths::ProjectPlatformExtensionDir(*PlatformDir), TEXT("Config/DataDrivenPlatformInfo.ini"));
		if (IFileManager::Get().FileExists(*IniPath))
		{
			DataDrivenIniFilenames.AddUnique(IniPath); // need AddUnique because if there's no project specified then ProjectConfigDir will be EngineConfigDir
		}
	}

	return DataDrivenIniFilenames;
}

static const TArray<FString> GetHostAndTargetDataDrivenIniFilenames()
{
	// grab all we find on disk, and filter only the Host/Target, likely a better way to only search on disk for our host/target but start with easier for now
	TArray<FString> DataDrivenIniFilenames = GetAllDataDrivenIniFilenames();

	FString TargetPlatformCmdLine;
	if (FParse::Value(FCommandLine::Get(), TEXT("TargetPlatform="), TargetPlatformCmdLine) && TargetPlatformCmdLine != TEXT("All"))
	{
		TSet<FString> HostAndTargetPlatforms;
		HostAndTargetPlatforms.Add(FString::Printf(TEXT("/%s/"), ANSI_TO_TCHAR(FPlatformProperties::IniPlatformName()))); // always add Host

		TArray<FString> TargetPlatformNames;
		TargetPlatformCmdLine.ParseIntoArray(TargetPlatformNames, TEXT("+"), true);

		for (const FString& TargetPlatformName : TargetPlatformNames)
		{
			FString Stripped = TargetPlatformName;
			Stripped.RemoveFromEnd("Client");
			Stripped.RemoveFromEnd("Server");
			Stripped.RemoveFromEnd("Editor");
			// Cook flavours, for e.g. Android, are defined as Android_ASTC etc.
			int32 UnderscoreIdx = INDEX_NONE;
			if (Stripped.FindChar(TEXT('_'), UnderscoreIdx))
			{
				Stripped.LeftInline(UnderscoreIdx);
			}

			// add any TargetPlatforms
			HostAndTargetPlatforms.Add(FString::Printf(TEXT("/%s/"), *Stripped));
		}

		TSet<FString> AllPlatforms(HostAndTargetPlatforms);
		TSet<FString> PlatformsToCheck = HostAndTargetPlatforms;

		while (PlatformsToCheck.Num() > 0)
		{
			TSet<FString> NewParentPlatforms;

			for (const FString& DataDrivenIniFilename : DataDrivenIniFilenames)
			{
				for (const FString& Platform : PlatformsToCheck)
				{
					if (DataDrivenIniFilename.Contains(Platform))
					{
						FString ParentIniPlatform = GetParentIniFromDDPI(DataDrivenIniFilename);
						if (!ParentIniPlatform.IsEmpty() && !AllPlatforms.Contains(ParentIniPlatform))
						{
							FString ParentIniWithSlash = FString(TEXT("/") + ParentIniPlatform + TEXT("/"));
							AllPlatforms.Add(ParentIniWithSlash);
							NewParentPlatforms.Add(ParentIniWithSlash);
						}
						break;
					}
				}
			}

			PlatformsToCheck = MoveTemp(NewParentPlatforms);
		}

		// remove any paths that are not part our Host or TargetPlatform list
		DataDrivenIniFilenames.RemoveAll([&AllPlatforms](const FString& Filename)
		{
			for (const FString& Component : AllPlatforms)
			{
				if (Filename.Contains(Component))
				{
					return false;
				}
			}
			return true;
		});
	}

	return DataDrivenIniFilenames;
}

static const TArray<FString>& GetDataDrivenIniFilenames()
{
	static bool bHasSearchedForFiles = false;
	static TArray<FString> DataDrivenIniFilenames;

	if (bHasSearchedForFiles == false)
	{
		bHasSearchedForFiles = true;

		if (FParse::Param(FCommandLine::Get(), TEXT("-LoadOnlyHostAndTargetPlatformModules")))
		{
			DataDrivenIniFilenames = GetHostAndTargetDataDrivenIniFilenames();
		}
		else
		{
			DataDrivenIniFilenames = GetAllDataDrivenIniFilenames();
		}
	}

	return DataDrivenIniFilenames;
}

int32 FDataDrivenPlatformInfoRegistry::GetNumDataDrivenIniFiles()
{
	return GetDataDrivenIniFilenames().Num();
}

bool FDataDrivenPlatformInfoRegistry::LoadDataDrivenIniFile(int32 Index, FConfigFile& IniFile, FString& PlatformName)
{
	const TArray<FString>& IniFilenames = GetDataDrivenIniFilenames();
	if (Index < 0 || Index >= IniFilenames.Num())
	{
		return false;
	}

	// manually load a FConfigFile object from a source ini file so that we don't do any SavedConfigDir processing or anything
	// (there's a possibility this is called before the ProjectDir is set)
	FString IniContents;
	if (FFileHelper::LoadFileToString(IniContents, *IniFilenames[Index]))
	{
		IniFile.ProcessInputFileContents(IniContents, IniFilenames[Index]);

		// platform extension paths are different (engine/platforms/platform/config, not engine/config/platform)
		if (IniFilenames[Index].StartsWith(FPaths::EnginePlatformExtensionDir(TEXT("")).TrimChar('/')))
		{
			PlatformName = FPaths::GetCleanFilename(FPaths::GetPath(FPaths::GetPath(IniFilenames[Index])));
		}
		else if (IniFilenames[Index].StartsWith(FPaths::ProjectPlatformExtensionDir(TEXT("")).TrimChar('/')))
		{
			PlatformName = FPaths::GetCleanFilename(FPaths::GetPath(FPaths::GetPath(IniFilenames[Index])));
		}
		else
		{
			// this could be 'Engine' for a shared DataDrivenPlatformInfo file
			PlatformName = FPaths::GetCleanFilename(FPaths::GetPath(IniFilenames[Index]));
		}

		IniFile.PlatformName = PlatformName;
		IniFile.bHasPlatformName = !PlatformName.IsEmpty();

		return true;
	}

	return false;
}

static void DDPIIniRedirect(FString& StringData)
{
	TArray<FString> Tokens;
	StringData.ParseIntoArray(Tokens, TEXT(":"));
	if (Tokens.Num() != 5)
	{
		StringData = TEXT("");
		return;
	}

	// now load a local version of the ini hierarchy
	FConfigFile LocalIni;
	FConfigCacheIni::LoadLocalIniFile(LocalIni, *Tokens[1], true, *Tokens[2]);

	// and get the platform's value (if it's not found, return an empty string)
	FString FoundValue;
	LocalIni.GetString(*Tokens[3], *Tokens[4], FoundValue);
	StringData = FoundValue;
}

// used to quickly check for commandline override
static FString GCommandLinePrefix;
static FString DDPITryRedirect(const FConfigFile& IniFile, const TCHAR* Key, bool* OutHadBang=nullptr)
{
	// check for commandline param
	if (GCommandLinePrefix.Len() > 0)
	{
		FString CmdLineValue;
		if (FParse::Value(FCommandLine::Get(), *(GCommandLinePrefix + Key + TEXT("=")), CmdLineValue))
		{
			UE_LOGF(LogTemp, Display, "--> Overriding DDPI setting %ls for to %ls (via prefix %ls)", Key, *CmdLineValue, *GCommandLinePrefix);
			return CmdLineValue;
		}
	}

	FString StringData;
	bool bWasFound = false;
	if ((bWasFound = IniFile.GetString(TEXT("DataDrivenPlatformInfo"), Key, StringData)) == false)
	{
		bWasFound = IniFile.GetString(TEXT("DataDrivenPlatformInfo"), *FString::Printf(TEXT("%s:%s"), ANSI_TO_TCHAR(FPlatformProperties::IniPlatformName()), Key), StringData);
	}
	if (bWasFound)
	{
		if (StringData.StartsWith(TEXT("ini:")) || StringData.StartsWith(TEXT("!ini:")))
		{
			// check for !'ing a bool
			if (OutHadBang != nullptr)
			{
				*OutHadBang = StringData[0] == TEXT('!');
			}

			// replace the string, overwriting it
			DDPIIniRedirect(StringData);
		}
	}
	return StringData;
}

static void DDPIGetBool(const FConfigFile& IniFile, const TCHAR* Key, bool& OutBool)
{
	bool bHadNot = false;
	FString StringData = DDPITryRedirect(IniFile, Key, &bHadNot);

	// if we ended up with a string, convert it, otherwise leave it alone
	if (StringData.Len() > 0)
	{
		OutBool = bHadNot ? !StringData.ToBool() : StringData.ToBool();
	}
}

static void DDPIGetInt(const FConfigFile& IniFile, const TCHAR* Key, int32& OutInt)
{
	FString StringData = DDPITryRedirect(IniFile, Key);

	// if we ended up with a string, convert it, otherwise leave it alone
	if (StringData.Len() > 0)
	{
		OutInt = FCString::Atoi(*StringData);
	}
}

static void DDPIGetUInt(const FConfigFile& IniFile, const TCHAR* Key, uint32& OutInt)
{
	FString StringData = DDPITryRedirect(IniFile, Key);

	// if we ended up with a string, convert it, otherwise leave it alone
	if (StringData.Len() > 0)
	{
		OutInt = (uint32)FCString::Strtoui64(*StringData, nullptr, 10);
	}
}

static void DDPIGetName(const FConfigFile& IniFile, const TCHAR* Key, FName& OutName)
{
	FString StringData = DDPITryRedirect(IniFile, Key);

	// if we ended up with a string, convert it, otherwise leave it alone
	if (StringData.Len() > 0)
	{
		OutName = FName(*StringData);
	}
}

static void DDPIGetString(const FConfigFile& IniFile, const TCHAR* Key, FString& OutString)
{
	FString StringData = DDPITryRedirect(IniFile, Key);

	// if we ended up with a string, convert it, otherwise leave it alone
	if (StringData.Len() > 0)
	{
		OutString = StringData;
	}
}

static void DDPIGetGuid(const FConfigFile& IniFile, const TCHAR* Key, FGuid& OutGuid)
{
	FString StringData = DDPITryRedirect(IniFile, Key);

	// if we ended up with a string, convert it, otherwise leave it alone
	if (StringData.Len() > 0)
	{
		OutGuid = FGuid(StringData);
	}
}

static void DDPIGetStringArray(const FConfigFile& IniFile, const TCHAR* Key, TArray<FString>& OutArray)
{
	// we don't support redirecting arrays
	IniFile.GetArray(TEXT("DataDrivenPlatformInfo"), Key, OutArray);
}

// Gets a string from a section, or empty string if it didn't exist
static FString GetSectionString(const FConfigSection& Section, FName Key)
{
	const FConfigValue* Value = Section.Find(Key);
	return Value ? Value->GetValue() : FString();
}

// Gets a string from a section, or default string if it didn't exist
static inline FName GetSectionString(const FConfigSection& Section, FName Key, FName DefaultValue)
{
	const FConfigValue* Value = Section.Find(Key);
	return Value ? *Value->GetValue() : DefaultValue;
}

#if DDPI_HAS_EXTENDED_PLATFORMINFO_DATA
static void ParsePreviewPlatforms(const FConfigFile& IniFile)
{
#if UE_ENABLE_CONFIG_SYSTEM
	if (!FParse::Param(FCommandLine::Get(), TEXT("NoPreviewPlatforms")))
	{
		// walk over the file looking for PreviewPlatform sections
		for (const auto& Section : IniFile)
		{
			if (Section.Key.StartsWith(TEXT("PreviewPlatform ")))
			{
				const FString& SectionName = Section.Key;

				// Early-out if enabled / disabled cvar is specified and not set (or the other way around)
				auto ShouldEarlyOut = [&](const FName& KeyName, bool bValueToEarlyOut) -> bool
					{
						TArray<FString> Tokens;
						GetSectionString(Section.Value, KeyName).ParseIntoArray(Tokens, TEXT(":"));
						if (Tokens.Num() == 5)
						{
							// now load a local version of the ini hierarchy
							FConfigFile LocalIni;
							FConfigCacheIni::LoadLocalIniFile(LocalIni, *Tokens[1], true, *Tokens[2]);

							// and get the enabled cvar's value
							bool bEnabled = false;
							LocalIni.GetBool(*Tokens[3], *Tokens[4], bEnabled);
							if (bEnabled == bValueToEarlyOut)
							{
								return true;
							}
						}
						return false;
					};
					
				if (ShouldEarlyOut(FName("EnabledCVar"), false) || ShouldEarlyOut(FName("DisabledCVar"), true))
				{
					continue;
				}

				FName PlatformName = *GetSectionString(Section.Value, FName("PlatformName"));
				checkf(PlatformName != NAME_None, TEXT("DataDrivenPlatformInfo section [%s] must specify a PlatformName"), *SectionName);
				
				FPreviewPlatformMenuItem Item;
				Item.PlatformName = PlatformName;
				Item.ShaderPlatformToPreview = *GetSectionString(Section.Value, FName("ShaderPlatform"));
				checkf(Item.ShaderPlatformToPreview != NAME_None, TEXT("DataDrivenPlatformInfo section [PreviewPlatform %s] must specify a ShaderPlatformToPreview"), *SectionName);

				FName PreviewPlatformName = FName(Item.ShaderPlatformToPreview.ToString() + TEXT("_Preview"));
				Item.PreviewShaderPlatformName = GetSectionString(Section.Value, FName("PreviewShaderPlatform"), PreviewPlatformName);
				Item.ShaderFormat = *GetSectionString(Section.Value, FName("ShaderFormat"));
				checkf(Item.ShaderFormat != NAME_None, TEXT("DataDrivenPlatformInfo section [PreviewPlatform %s] must specify a ShaderFormat"), *SectionName);
				Item.ActiveIconPath = GetSectionString(Section.Value, FName("ActiveIconPath"));
				Item.ActiveIconName = *GetSectionString(Section.Value, FName("ActiveIconName"));
				Item.InactiveIconPath = GetSectionString(Section.Value, FName("InactiveIconPath"));
				Item.InactiveIconName = *GetSectionString(Section.Value, FName("InactiveIconName"));
				
				Item.PreviewFeatureLevelName = *GetSectionString(Section.Value, FName("PreviewFeatureLevel"));

				
				FTextStringHelper::ReadFromBuffer(*GetSectionString(Section.Value, FName("MenuTooltip")), Item.MenuTooltip);
				FTextStringHelper::ReadFromBuffer(*GetSectionString(Section.Value, FName("IconText")), Item.IconText);


				FString AllDeviceProfiles = GetSectionString(Section.Value, FName("DeviceProfileName"));
				FString AllFriendlyName = GetSectionString(Section.Value, FName("FriendlyName"));
				TArray<FString> DeviceProfileNames, FriendlyNames;
				AllDeviceProfiles.ParseIntoArray(DeviceProfileNames, TEXT(":"));
				AllFriendlyName.ParseIntoArray(FriendlyNames, TEXT(":"));

				if (DeviceProfileNames.Num() == 0)
				{
					DeviceProfileNames.Add(TEXT(""));
					FriendlyNames.Add(TEXT(""));
				}

				for (int DPIndex = 0; DPIndex < DeviceProfileNames.Num(); DPIndex++)
				{
					Item.DeviceProfileName = *DeviceProfileNames[DPIndex].TrimStartAndEnd();
					if (DPIndex < FriendlyNames.Num())
					{
						FTextStringHelper::ReadFromBuffer(*FriendlyNames[DPIndex].TrimStartAndEnd(), Item.OptionalFriendlyNameOverride);
					}
					else if (DeviceProfileNames.Num() > 1)
					{
						Item.OptionalFriendlyNameOverride = FText::FromString(Item.DeviceProfileName.ToString());
					}
									
					ShaderPlatformNameToMenuItem.FindOrAdd(Item.ShaderPlatformToPreview).Add(Item);
					PreviewPlatformMenuItems.Add(Item);
				}
			}
		}
	}
#endif // UE_ENABLE_CONFIG_SYSTEM
}
#endif // DDPI_HAS_EXTENDED_PLATFORMINFO_DATA

static void LoadDDPIIniSettings(const FConfigFile& IniFile, FDataDrivenPlatformInfo& Info, FName PlatformName)
{
	// look if this platform has any overrides on the commandline
	FString CmdLinePrefix = FString::Printf(TEXT("ddpi:%s:"), *PlatformName.ToString());
	if (FCString::Strifind(FCommandLine::Get(), *CmdLinePrefix) != nullptr)
	{
		GCommandLinePrefix = CmdLinePrefix;
	}

	DDPIGetBool(IniFile, TEXT("bIsConfidential"), Info.bIsConfidential);
	DDPIGetBool(IniFile, TEXT("bIsFakePlatform"), Info.bIsFakePlatform);
	DDPIGetString(IniFile, TEXT("TargetSettingsIniSectionName"), Info.TargetSettingsIniSectionName);
	DDPIGetString(IniFile, TEXT("HardwareCompressionFormat"), Info.HardwareCompressionFormat);
	DDPIGetString(IniFile, TEXT("ICUDataVersion"), Info.ICUDataVersion);
	DDPIGetStringArray(IniFile, TEXT("AdditionalRestrictedFolders"), Info.AdditionalRestrictedFolders);

	DDPIGetBool(IniFile, TEXT("Freezing_b32Bit"), Info.Freezing_b32Bit);
	DDPIGetUInt(IniFile, Info.Freezing_b32Bit ? TEXT("Freezing_MaxFieldAlignment32") : TEXT("Freezing_MaxFieldAlignment64"), Info.Freezing_MaxFieldAlignment);
	DDPIGetBool(IniFile, TEXT("Freezing_bForce64BitMemoryImagePointers"), Info.Freezing_bForce64BitMemoryImagePointers);
	DDPIGetBool(IniFile, TEXT("Freezing_bAlignBases"), Info.Freezing_bAlignBases);

	DDPIGetGuid(IniFile, TEXT("GlobalIdentifier"), Info.GlobalIdentifier);
	checkf(Info.GlobalIdentifier != FGuid(), TEXT("Platform %s didn't have a valid GlobalIdentifier set in DataDrivenPlatformInfo.ini"), *PlatformName.ToString());

	// NOTE: add more settings here!
	DDPIGetBool(IniFile, TEXT("bHasDedicatedGamepad"), Info.bHasDedicatedGamepad);
	DDPIGetBool(IniFile, TEXT("bDefaultInputStandardKeyboard"), Info.bDefaultInputStandardKeyboard);

	DDPIGetBool(IniFile, TEXT("bInputSupportConfigurable"), Info.bInputSupportConfigurable);
	DDPIGetString(IniFile, TEXT("DefaultInputType"), Info.DefaultInputType);
	DDPIGetBool(IniFile, TEXT("bSupportsMouseAndKeyboard"), Info.bSupportsMouseAndKeyboard);
	DDPIGetBool(IniFile, TEXT("bSupportsGamepad"), Info.bSupportsGamepad);
	DDPIGetBool(IniFile, TEXT("bCanChangeGamepadType"), Info.bCanChangeGamepadType);
	DDPIGetBool(IniFile, TEXT("bSupportsTouch"), Info.bSupportsTouch);

	DDPIGetName(IniFile, TEXT("OverrideCookPlatformName"), Info.OverrideCookPlatformName);

#if DDPI_HAS_EXTENDED_PLATFORMINFO_DATA

	DDPIGetString(IniFile, TEXT("AutoSDKPath"), Info.AutoSDKPath);
	DDPIGetString(IniFile, TEXT("TutorialPath"), Info.SDKTutorial);
	DDPIGetName(IniFile, TEXT("PlatformGroupName"), Info.PlatformGroupName);
	DDPIGetName(IniFile, TEXT("PlatformSubMenu"), Info.PlatformSubMenu);


	DDPIGetString(IniFile, TEXT("NormalIconPath"), Info.IconPaths.NormalPath);
	DDPIGetString(IniFile, TEXT("LargeIconPath"), Info.IconPaths.LargePath);
	DDPIGetString(IniFile, TEXT("XLargeIconPath"), Info.IconPaths.XLargePath);
	if (Info.IconPaths.XLargePath == TEXT(""))
	{
		Info.IconPaths.XLargePath = Info.IconPaths.LargePath;
	}

	FString PlatformString = PlatformName.ToString();
	Info.IconPaths.NormalStyleName = *FString::Printf(TEXT("Launcher.Platform_%s"), *PlatformString);
	Info.IconPaths.LargeStyleName = *FString::Printf(TEXT("Launcher.Platform_%s.Large"), *PlatformString);
	Info.IconPaths.XLargeStyleName = *FString::Printf(TEXT("Launcher.Platform_%s.XLarge"), *PlatformString);

	Info.bCanUseCrashReporter = true; // not specified means true, not false
	DDPIGetBool(IniFile, TEXT("bCanUseCrashReporter"), Info.bCanUseCrashReporter);
	DDPIGetBool(IniFile, TEXT("bUsesHostCompiler"), Info.bUsesHostCompiler);
	DDPIGetBool(IniFile, TEXT("bUATClosesAfterLaunch"), Info.bUATClosesAfterLaunch);
	DDPIGetBool(IniFile, TEXT("bIsEnabled"), Info.bEnabledForUse);

	DDPIGetName(IniFile, TEXT("UBTPlatformName"), Info.UBTPlatformName);
	// if unspecified, use the ini platform name (only Win64 breaks this)
	if (Info.UBTPlatformName == NAME_None)
	{
		Info.UBTPlatformName = PlatformName;
	}
	Info.UBTPlatformString = Info.UBTPlatformName.ToString();
		
	GCommandLinePrefix = TEXT("");

	// now that we have all targetplatforms in a single TP module per platform, just look for it (or a ShaderFormat for other tools that may want this)
	// we could look for Platform*, but then platforms that are a substring of another one could return a false positive (Windows* would find Windows31TargetPlatform)
	Info.bHasCompiledTargetSupport = FDataDrivenPlatformInfoRegistry::HasCompiledSupportForPlatform(PlatformName, FDataDrivenPlatformInfoRegistry::EPlatformNameType::TargetPlatform);

	// Skip Preview Platform parsing if the Platforms are disabled
	// Exception: Fake Platforms that have no real ShaderPlatform and only PreviewShaderPlatform for previewing or fake Platforms (e.g. VulkanPC).
	auto IniHasRealShaderPlatforms = [&IniFile]()
	{
		for (const auto& Section : IniFile)
		{
			if (Section.Key.StartsWith(TEXT("ShaderPlatform ")))
			{
				return true;
			}
		}
		return false;
	};

	if (Info.bEnabledForUse || Info.bIsFakePlatform || !IniHasRealShaderPlatforms())
	{
		ParsePreviewPlatforms(IniFile);
	}
#endif
}

/**
* Get the global set of data driven platform information
*/
const TMap<FName, FDataDrivenPlatformInfo>& FDataDrivenPlatformInfoRegistry::GetAllPlatformInfos()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDataDrivenPlatformInfoRegistry::GetAllPlatformInfos);

	static bool bHasSearchedForPlatforms = false;

	// look on disk for special files
	if (bHasSearchedForPlatforms == false)
	{
		bHasSearchedForPlatforms = true;

		int32 NumFiles = FDataDrivenPlatformInfoRegistry::GetNumDataDrivenIniFiles();

		TMap<FString, FString> IniParents;
		for (int32 Index = 0; Index < NumFiles; Index++)
		{
			// load the .ini file
			FConfigFile IniFile;
			FString PlatformString;
			FDataDrivenPlatformInfoRegistry::LoadDataDrivenIniFile(Index, IniFile, PlatformString);

			FName PlatformName(*PlatformString);
			// platform info is registered by the platform name
			if (IniFile.Contains(TEXT("DataDrivenPlatformInfo")))
			{
				// cache info
				FDataDrivenPlatformInfo& Info = DataDrivenPlatforms.FindOrAdd(PlatformName, FDataDrivenPlatformInfo());
				LoadDDPIIniSettings(IniFile, Info, PlatformName);
				Info.IniPlatformName = PlatformName;

				// get the parent to build list later
				FString IniParent;
				bool bSettingPresent = IniFile.GetString(TEXT("DataDrivenPlatformInfo"), TEXT("IniParent"), IniParent);
				// we want an empty string in here if there is nothing already in it for the platform, however if we have a project
				// DDPI.ini file, and it doesn't have the setting, it should not overwrite the good value with empty string
				// so, add empty string if not in map, but only overwrite the engine value if the project value was present (even
				// if that value is empty string: "IniParent=")
				if (!IniParents.Contains(PlatformString) || bSettingPresent)
				{
					IniParents.Add(PlatformString, IniParent);
				}

				// get platform name aliases
				FString PlatformNameAliasesStr;
				IniFile.GetString(TEXT("DataDrivenPlatformInfo"), TEXT("PlatformNameAliases"), PlatformNameAliasesStr);
				TArray<FString> PlatformNameAliases;
				PlatformNameAliasesStr.ParseIntoArrayWS(PlatformNameAliases, TEXT(","));
				for (const FString& PlatformNameAlias : PlatformNameAliases)
				{
					GlobalPlatformNameAliases.Add(*PlatformNameAlias, PlatformName);
				}
			}
		}

		// now that all are read in, calculate the ini parent chain, starting with parent-most
		for (auto& It : DataDrivenPlatforms)
		{
			// walk up the chain and build up the ini chain of parents
			for (FString CurrentPlatform = IniParents.FindRef(It.Key.ToString()); CurrentPlatform != TEXT(""); CurrentPlatform = IniParents.FindRef(CurrentPlatform))
			{
				// insert at 0 to reverse the order
				It.Value.IniParentChain.Insert(CurrentPlatform, 0);
			}
		}

		DataDrivenPlatforms.GetKeys(AllSortedPlatformNames);
		// now sort them into arrays of keys and values
		Algo::Sort(AllSortedPlatformNames, [](FName One, FName Two) -> bool
		{
			return One.Compare(Two) < 0;
		});
		
		// now remove the invalid platforms (this is not about installed SDKs or anything, just based on ini values)
		SortedPlatformNames = AllSortedPlatformNames;
		SortedPlatformNames.RemoveAll([](FName Platform)
		{
			return DataDrivenPlatforms[Platform].bIsFakePlatform;
		});


		// now build list of values from the sort
		AllSortedPlatformInfos.AddZeroed(AllSortedPlatformNames.Num());
		for (int Index = 0; Index < AllSortedPlatformInfos.Num(); Index++)
		{
			AllSortedPlatformInfos[Index] = &DataDrivenPlatforms[AllSortedPlatformNames[Index]];
		}

		SortedPlatformInfos.AddZeroed(SortedPlatformNames.Num());
		for (int Index = 0; Index < SortedPlatformInfos.Num(); Index++)
		{
			SortedPlatformInfos[Index] = &DataDrivenPlatforms[SortedPlatformNames[Index]];
		}
	}

	return DataDrivenPlatforms;
}

const TArray<FName> FDataDrivenPlatformInfoRegistry::GetSortedPlatformNames(EPlatformInfoType PlatformType)
{
	// make sure we've read in the inis
	GetAllPlatformInfos();

	return PlatformType == EPlatformInfoType::AllPlatformInfos ? AllSortedPlatformNames : SortedPlatformNames;
}

const TArray<const FDataDrivenPlatformInfo*>& FDataDrivenPlatformInfoRegistry::GetSortedPlatformInfos(EPlatformInfoType PlatformType)
{
	// make sure we've read in the inis
	GetAllPlatformInfos();

	return PlatformType == EPlatformInfoType::AllPlatformInfos ? AllSortedPlatformInfos : SortedPlatformInfos;
}

const TArray<FString>& FDataDrivenPlatformInfoRegistry::GetPlatformDirectoryNames(bool bCheckValid)
{
	static bool bHasSearchedForPlatforms[2] = { false, false };
	static TArray<FString> PlatformDirectories[2];

	// Which PlatformDirectories cache to use
	int32 CacheIndex = bCheckValid ? 1 : 0;

	if (bHasSearchedForPlatforms[CacheIndex] == false)
	{
		bHasSearchedForPlatforms[CacheIndex] = true;

		// look for possible platforms
		const TMap<FName, FDataDrivenPlatformInfo>& Infos = GetAllPlatformInfos();
		for (auto Pair : Infos)
		{
#if DDPI_HAS_EXTENDED_PLATFORMINFO_DATA
			// if the editor hasn't compiled in support for the platform, it's not "valid"
			if (bCheckValid && !HasCompiledSupportForPlatform(Pair.Key, EPlatformNameType::Ini))
			{
				continue;
			}
#endif

			// add ourself as valid
			PlatformDirectories[CacheIndex].AddUnique(Pair.Key.ToString());

			// now add additional directories
			for (FString& AdditionalDir : Pair.Value.AdditionalRestrictedFolders)
			{
				PlatformDirectories[CacheIndex].AddUnique(AdditionalDir);
			}
		}
	}

	return PlatformDirectories[CacheIndex];
}

const FDataDrivenPlatformInfo& FDataDrivenPlatformInfoRegistry::GetPlatformInfo(FName PlatformName)
{
	const FDataDrivenPlatformInfo* Info = GetAllPlatformInfos().Find(PlatformName);
	static FDataDrivenPlatformInfo Empty;
	if (Info == nullptr && GlobalPlatformNameAliases.Contains(PlatformName))
	{
		Info = GetAllPlatformInfos().Find(GlobalPlatformNameAliases.FindChecked(PlatformName));
	}
	return Info ? *Info : Empty;
}

const FDataDrivenPlatformInfo& FDataDrivenPlatformInfoRegistry::GetPlatformInfo(const FString& PlatformName)
{
	return GetPlatformInfo(FName(*PlatformName));
}

const FDataDrivenPlatformInfo& FDataDrivenPlatformInfoRegistry::GetPlatformInfo(const char* PlatformName)
{
	return GetPlatformInfo(FName(PlatformName));
}


const TArray<FName>& FDataDrivenPlatformInfoRegistry::GetConfidentialPlatforms()
{
	static bool bHasSearchedForPlatforms = false;
	static TArray<FName> FoundPlatforms;

	// look on disk for special files
	if (bHasSearchedForPlatforms == false)
	{
		for (auto It : GetAllPlatformInfos())
		{
			if (It.Value.bIsConfidential)
			{
				FoundPlatforms.Add(It.Key);
			}
		}

		bHasSearchedForPlatforms = true;
	}

	// return whatever we have already found
	return FoundPlatforms;
}


#if DDPI_HAS_EXTENDED_PLATFORMINFO_DATA
bool FDataDrivenPlatformInfoRegistry::HasCompiledSupportForPlatform(FName PlatformName, EPlatformNameType PlatformNameType)
{
	if (PlatformNameType == EPlatformNameType::Ini)
	{
		// get the DDPI info object
		const FDataDrivenPlatformInfo& Info = GetPlatformInfo(PlatformName);
		return Info.bHasCompiledTargetSupport;
	}
	else if (PlatformNameType == EPlatformNameType::UBT)
	{
		// find all the DataDrivenPlatformInfo objects and find a matching the UBT name
		for (auto& Pair : FDataDrivenPlatformInfoRegistry::GetAllPlatformInfos())
		{
			// if this platform matches the UBT platform name, check it's Ini name
			if (Pair.Value.UBTPlatformName == PlatformName)
			{
				return HasCompiledSupportForPlatform(Pair.Key, EPlatformNameType::Ini);
			}
		}

		return false;
	}
	else if (PlatformNameType == EPlatformNameType::TargetPlatform)
	{
		// was this TP compiled, or a shaderformat (useful for SCW if it ever calls this)
		// if this is a program that doesn't use TargetPlatform, then we don't need to know if one particular 
		// TargetPlatform is missing support compiled in, because no platforms are "compiled in"
		return
			!FModuleManager::Get().ModuleExists(TEXT("TargetPlatform")) ||
			FModuleManager::Get().ModuleExists(*FString::Printf(TEXT("%sTargetPlatform"), *PlatformName.ToString())) || 
			FModuleManager::Get().ModuleExists(*FString::Printf(TEXT("%sShaderFormat"), *PlatformName.ToString()));
	}

	return false;
}

const TArray<struct FPreviewPlatformMenuItem>& FDataDrivenPlatformInfoRegistry::GetAllPreviewPlatformMenuItems()
{
	return PreviewPlatformMenuItems;
}

const TArray<struct FPreviewPlatformMenuItem>* FDataDrivenPlatformInfoRegistry::GetMenuItemForShaderPlatformName(FName ShaderPlatformName)
{
	return ShaderPlatformNameToMenuItem.Find(ShaderPlatformName);
}

bool FDataDrivenPlatformInfoRegistry::IsPlatformHiddenFromUI(FName PlatformName)
{
	return PlatformsHiddenFromUI.Contains(PlatformName);
}

void FDataDrivenPlatformInfoRegistry::SetPlatformHiddenFromUI(FName PlatformName)
{
	PlatformsHiddenFromUI.Add(PlatformName);
}


#endif
