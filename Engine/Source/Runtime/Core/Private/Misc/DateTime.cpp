// Copyright Epic Games, Inc. All Rights Reserved.
#include "Misc/DateTime.h"
#include "CoreGlobals.h"
#include "HAL/PlatformTime.h"
#include "Templates/TypeHash.h"
#include "UObject/PropertyPortFlags.h"
#include "Misc/CoreDelegates.h"

#if USE_ESTIMATED_UTCNOW
#include "HAL/IConsoleManager.h"

static TAutoConsoleVariable<int32> CVarEstimatedUtcNowRebaseTimeSeconds(
	TEXT("time.EstimatedUtcNowRebaseTimeSeconds"),
	600,
	TEXT("Number of seconds before rebasing EstimatedUtcNow() "),
	ECVF_ReadOnly
);

static volatile int32 RebaseEstimatedUtcNowViaDelegate = false;
#endif // #if USE_ESTIMATED_UTCNOW

/* FDateTime constants
 *****************************************************************************/

const int32 FDateTime::DaysPerMonth[] = { 0, 31, 28, 31, 30, 31, 30, 31, 31, 30, 31, 30, 31 };
const int32 FDateTime::DaysToMonth[] = { 0, 31, 59, 90, 120, 151, 181, 212, 243, 273, 304, 334, 365 };

const TCHAR* FDateTime::ShortDayNames[] = { TEXT("Mon"), TEXT("Tue"), TEXT("Wed"), TEXT("Thu"), TEXT("Fri"), TEXT("Sat"), TEXT("Sun") };
const TCHAR* FDateTime::LongDayNames[] = { TEXT("Monday"), TEXT("Tuesday"), TEXT("Wednesday"), TEXT("Thursday"), TEXT("Friday"), TEXT("Saturday"), TEXT("Sunday") };

const TCHAR* FDateTime::ShortMonthNames[] = { TEXT("Jan"), TEXT("Feb"), TEXT("Mar"), TEXT("Apr"), TEXT("May"), TEXT("Jun"), TEXT("Jul"), TEXT("Aug"), TEXT("Sep"), TEXT("Oct"), TEXT("Nov"), TEXT("Dec") };
const TCHAR* FDateTime::LongMonthNames[] = { TEXT("January"), TEXT("February"), TEXT("March"), TEXT("April"), TEXT("May"), TEXT("June"), TEXT("July"), TEXT("August"), TEXT("September"), TEXT("October"), TEXT("November"), TEXT("December") };


/* FDateTime structors
 *****************************************************************************/

FDateTime::FDateTime(int32 Year, int32 Month, int32 Day, int32 Hour, int32 Minute, int32 Second, int32 Millisecond)
{
	if (!Validate(Year, Month, Day, Hour, Minute, Second, Millisecond))
	{
		UE_LOGF(LogCore, Fatal, "Invalid Date values. Y:%d, M:%d, D:%d, H:%d, M:%d, S:%d, Ms:%d",
			Year, Month, Day, Hour, Minute, Second, Millisecond);
	}

	int32 TotalDays = 0;

	if ((Month > 2) && IsLeapYear(Year))
	{
		++TotalDays;
	}

	--Year;											// the current year is not a full year yet
	--Month;										// the current month is not a full month yet

	TotalDays += Year * 365;
	TotalDays += Year / 4;							// leap year day every four years...
	TotalDays -= Year / 100;						// ...except every 100 years...
	TotalDays += Year / 400;						// ...but also every 400 years
	TotalDays += DaysToMonth[Month];				// days in this year up to last month
	TotalDays += Day - 1;							// days in this month minus today

	Ticks = TotalDays * ETimespan::TicksPerDay
		+ Hour * ETimespan::TicksPerHour
		+ Minute * ETimespan::TicksPerMinute
		+ Second * ETimespan::TicksPerSecond
		+ Millisecond * ETimespan::TicksPerMillisecond;
}


/* FDateTime interface
 *****************************************************************************/

bool FDateTime::ExportTextItem(FString& ValueStr, FDateTime const& DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const
{
	ValueStr += ToString();

	return true;
}


void FDateTime::GetDate(int32& OutYear, int32& OutMonth, int32& OutDay) const
{
	// Based on FORTRAN code in:
	// Fliegel, H. F. and van Flandern, T. C.,
	// Communications of the ACM, Vol. 11, No. 10 (October 1968).

	int32 i, j, k, l, n;

	l = FMath::FloorToInt32(GetDate().GetJulianDay() + 0.5) + 68569;
	n = 4 * l / 146097;
	l = l - (146097 * n + 3) / 4;
	i = 4000 * (l + 1) / 1461001;
	l = l - 1461 * i / 4 + 31;
	j = 80 * l / 2447;
	k = l - 2447 * j / 80;
	l = j / 11;
	j = j + 2 - 12 * l;
	i = 100 * (n - 49) + i + l;

	OutYear = i;
	OutMonth = j;
	OutDay = k;
}


int32 FDateTime::GetDay() const
{
	int32 Year, Month, Day;
	GetDate(Year, Month, Day);

	return Day;
}


EDayOfWeek FDateTime::GetDayOfWeek() const
{
	// January 1, 0001 was a Monday
	return static_cast<EDayOfWeek>((Ticks / ETimespan::TicksPerDay) % 7);
}


int32 FDateTime::GetDayOfYear() const
{
	int32 Year, Month, Day;
	GetDate(Year, Month, Day);

	for (int32 CurrentMonth = 1; CurrentMonth < Month; ++CurrentMonth)
	{
		Day += DaysInMonth(Year, CurrentMonth);
	}

	return Day;
}


int32 FDateTime::GetHour12() const
{
	int32 Hour = GetHour();

	if (Hour < 1)
	{
		return 12;
	}

	if (Hour > 12)
	{
		return (Hour - 12);
	}

	return Hour;
}


int32 FDateTime::GetMonth() const
{
	int32 Year, Month, Day;
	GetDate(Year, Month, Day);

	return Month;
}


int32 FDateTime::GetYear() const
{
	int32 Year, Month, Day;
	GetDate(Year, Month, Day);

	return Year;
}


bool FDateTime::ImportTextItem(const TCHAR*& Buffer, int32 PortFlags, UObject* Parent, FOutputDevice* ErrorText)
{
	const int32 ExportDateTimeLen = 19;

	if (FPlatformString::Strlen(Buffer) < ExportDateTimeLen)
	{
		return false;
	}

	if (!Parse(FString(Buffer).Left(ExportDateTimeLen), *this))
	{
		return false;
	}

	Buffer += ExportDateTimeLen;

	return true;
}


bool FDateTime::Serialize(FArchive& Ar)
{
	Ar << *this;

	return true;
}


bool FDateTime::NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess)
{
	Ar << *this;
	bOutSuccess = true;
	return true;
}

FString FDateTime::ToHttpDate() const
{
	FString Time = FString::Printf(TEXT("%02i:%02i:%02i"), GetHour(), GetMinute(), GetSecond());

	return FString::Printf(TEXT("%s, %02d %s %d %s GMT"), ShortDayNames[(int32)GetDayOfWeek()], GetDay(), ShortMonthNames[(int32)GetMonthOfYear() - 1], GetYear(), *Time);
}


FString FDateTime::ToIso8601() const
{
	return ToString(TEXT("%Y-%m-%dT%H:%M:%S.%sZ"));
}


FString FDateTime::ToString() const
{
	return ToString(TEXT("%Y.%m.%d-%H.%M.%S"));
}


FString FDateTime::ToString(const TCHAR* Format) const
{
	TStringBuilder<48> Result;
	ToString(Format, Result);
	return FString(Result);
}

void FDateTime::ToString(const TCHAR* Format, FStringBuilderBase& Result) const
{
	if (UNLIKELY(!Format))
	{
		return;
	}

	const auto AppendZeroPaddedInteger = [](FStringBuilderBase& Output, int32 Value, int32 DigitCount)
	{
		const int32 OutputIndex = Output.AddUninitialized(DigitCount);
		TCHAR* const OutputData = Output.GetData();
		for (; DigitCount--; Value /= 10)
		{
			OutputData[OutputIndex + DigitCount] = TEXT('0') + (Value % 10);
		}
	};

	int32 Year = 0, Month = 0, Day = 0;
	const auto GetLocalDateIfZero = [this, &Year, &Month, &Day](int32 Arg)
	{
		if (Arg == 0)
		{
			GetDate(Year, Month, Day);
		}
	};
	const auto GetLocalYear = [this, &Year, &GetLocalDateIfZero] { GetLocalDateIfZero(Year); return Year; };
	const auto GetLocalMonth = [this, &Month, &GetLocalDateIfZero] { GetLocalDateIfZero(Month); return Month; };
	const auto GetLocalDay = [this, &Day, &GetLocalDateIfZero] { GetLocalDateIfZero(Day); return Day; };

	for (; *Format; ++Format)
	{
		if (*Format == TCHAR('%') && *++Format)
		{
			switch (*Format)
			{
			case TCHAR('a'): Result.Append(IsMorning() ? TEXT("am") : TEXT("pm")); break;
			case TCHAR('A'): Result.Append(IsMorning() ? TEXT("AM") : TEXT("PM")); break;
			case TCHAR('D'): AppendZeroPaddedInteger(Result, GetDayOfYear(), 3); break;
			case TCHAR('d'): AppendZeroPaddedInteger(Result, GetLocalDay(), 2); break;
			case TCHAR('m'): AppendZeroPaddedInteger(Result, GetLocalMonth(), 2); break;
			case TCHAR('y'): AppendZeroPaddedInteger(Result, GetLocalYear() % 100, 2); break;
			case TCHAR('Y'): AppendZeroPaddedInteger(Result, GetLocalYear(), 4); break;
			case TCHAR('h'): AppendZeroPaddedInteger(Result, GetHour12(), 2); break;
			case TCHAR('H'): AppendZeroPaddedInteger(Result, GetHour(), 2); break;
			case TCHAR('M'): AppendZeroPaddedInteger(Result, GetMinute(), 2); break;
			case TCHAR('S'): AppendZeroPaddedInteger(Result, GetSecond(), 2); break;
			case TCHAR('s'): AppendZeroPaddedInteger(Result, GetMillisecond(), 3); break;
			default:         Result.AppendChar(*Format);
			}
		}
		else
		{
			Result.AppendChar(*Format);
		}
	}
}

FString FDateTime::ToFormattedString(const TCHAR* Format) const
{
	TStringBuilder<32> Result;

	if (Format != nullptr)
	{
		while (*Format != TCHAR('\0'))
		{
			if ((*Format == TCHAR('%')) && (*(++Format) != TCHAR('\0')))
			{
				switch (*Format)
				{
				case TCHAR('a'): Result.Append(ShortDayNames[(int32)GetDayOfWeek()]); break;
				case TCHAR('A'): Result.Append(LongDayNames[(int32)GetDayOfWeek()]); break;
				case TCHAR('w'): Result.Appendf(TEXT("%i"), ((int32)GetDayOfWeek() + 1) % 7); break;
				case TCHAR('y'): Result.Appendf(TEXT("%02i"), GetYear() % 100); break;
				case TCHAR('Y'): Result.Appendf(TEXT("%04i"), GetYear()); break;
				case TCHAR('b'): Result.Append(ShortMonthNames[(int32)GetMonthOfYear() - 1]); break;
				case TCHAR('B'): Result.Append(LongMonthNames[(int32)GetMonthOfYear() - 1]); break;
				case TCHAR('m'): Result.Appendf(TEXT("%02i"), GetMonth()); break;
				case TCHAR('d'): Result.Appendf(TEXT("%02i"), GetDay()); break;
				case TCHAR('e'): Result.Appendf(TEXT("%i"), GetDay()); break;
				case TCHAR('l'): Result.Appendf(TEXT("%i"), GetHour12()); break;
				case TCHAR('I'): Result.Appendf(TEXT("%02i"), GetHour12()); break;
				case TCHAR('H'): Result.Appendf(TEXT("%02i"), GetHour()); break;
				case TCHAR('M'): Result.Appendf(TEXT("%02i"), GetMinute()); break;
				case TCHAR('S'): Result.Appendf(TEXT("%02i"), GetSecond()); break;
				case TCHAR('p'): Result.Append(IsMorning() ? TEXT("AM") : TEXT("PM")); break;
				case TCHAR('P'): Result.Append(IsMorning() ? TEXT("am") : TEXT("pm")); break;
				case TCHAR('j'): Result.Appendf(TEXT("%03i"), GetDayOfYear()); break;
				default:		 Result.AppendChar(*Format);
				}
			}
			else
			{
				Result.AppendChar(*Format);
			}

			// move to the next one
			Format++;
		}
	}

	return FString(Result);
}


/* FDateTime static interface
 *****************************************************************************/

int32 FDateTime::DaysInMonth(int32 Year, int32 Month)
{
	check((Month >= 1) && (Month <= 12));

	if ((Month == 2) && IsLeapYear(Year))
	{
		return 29;
	}

	return DaysPerMonth[Month];
}


int32 FDateTime::DaysInYear(int32 Year)
{
	if (IsLeapYear(Year))
	{
		return 366;
	}

	return 365;
}


bool FDateTime::IsLeapYear(int32 Year)
{
	if ((Year % 4) == 0)
	{
		return (((Year % 100) != 0) || ((Year % 400) == 0));
	}

	return false;
}


FDateTime FDateTime::Now()
{
	int32 Year, Month, Day, DayOfWeek;
	int32 Hour, Minute, Second, Millisecond;

	FPlatformTime::SystemTime(Year, Month, DayOfWeek, Day, Hour, Minute, Second, Millisecond);

	return FDateTime(Year, Month, Day, Hour, Minute, Second, Millisecond);
}


bool FDateTime::Parse(const FString& DateTimeString, FDateTime& OutDateTime)
{
	// first replace -, : and . with space
	FString FixedString = DateTimeString.Replace(TEXT("-"), TEXT(" "));
	FixedString.ReplaceInline(TEXT(":"), TEXT(" "), ESearchCase::CaseSensitive);
	FixedString.ReplaceInline(TEXT("."), TEXT(" "), ESearchCase::CaseSensitive);

	// split up on a single delimiter
	TArray<FString> Tokens;
	FixedString.ParseIntoArray(Tokens, TEXT(" "), true);

	// make sure it parsed it properly (within reason)
	if ((Tokens.Num() < 6) || (Tokens.Num() > 7))
	{
		return false;
	}

	const int32 Year = FCString::Atoi(*Tokens[0]);
	const int32 Month = FCString::Atoi(*Tokens[1]);
	const int32 Day = FCString::Atoi(*Tokens[2]);
	const int32 Hour = FCString::Atoi(*Tokens[3]);
	const int32 Minute = FCString::Atoi(*Tokens[4]);
	const int32 Second = FCString::Atoi(*Tokens[5]);
	const int32 Millisecond = Tokens.Num() > 6 ? FCString::Atoi(*Tokens[6]) : 0;

	if (!Validate(Year, Month, Day, Hour, Minute, Second, Millisecond))
	{
		return false;
	}

	// convert the tokens to numbers
	OutDateTime.Ticks = FDateTime(Year, Month, Day, Hour, Minute, Second, Millisecond).Ticks;

	return true;
}


bool FDateTime::ParseHttpDate(const FString& HttpDate, FDateTime& OutDateTime)
{
	auto ParseTime = [](const FString& Time, int32& Hour, int32& Minute, int32& Second) -> bool
	{
		// 2DIGIT ":" 2DIGIT ":" 2DIGIT
		// ; 00:00 : 00 - 23 : 59 : 59
		TArray<FString> Tokens;

		// split up on a single delimiter
		int32 NumTokens = Time.ParseIntoArray(Tokens, TEXT(":"), true);

		if (NumTokens == 3)
		{
			Hour = FCString::Atoi(*Tokens[0]);
			Minute = FCString::Atoi(*Tokens[1]);
			Second = FCString::Atoi(*Tokens[2]);

			return (Hour >= 0 && Hour < 24) && (Minute >= 0 && Minute <= 59) && (Second >= 0 && Second <= 59);
		}

		return false;
	};

	auto ParseWkday = [](const FString& WkDay) -> int32
	{
		const int32 NumChars = WkDay.Len();

		if (NumChars == 3)
		{
			if (WkDay.Equals(TEXT("Mon")))
			{
				return 1;
			}
			else if (WkDay.Equals(TEXT("Tue")))
			{
				return 2;
			}
			else if (WkDay.Equals(TEXT("Wed")))
			{
				return 3;
			}
			else if (WkDay.Equals(TEXT("Thu")))
			{
				return 4;
			}
			else if (WkDay.Equals(TEXT("Fri")))
			{
				return 5;
			}
			else if (WkDay.Equals(TEXT("Sat")))
			{
				return 6;
			}
			else if (WkDay.Equals(TEXT("Sun")))
			{
				return 7;
			}
		}

		return -1;
	};

	auto ParseWeekday = [](const FString& WeekDay) -> int32
	{
		const int32 NumChars = WeekDay.Len();

		if (NumChars >= 6 && NumChars <= 9)
		{
			if (WeekDay.Equals(TEXT("Monday")))
			{
				return 1;
			}
			else if (WeekDay.Equals(TEXT("Tuesday")))
			{
				return 2;
			}
			else if (WeekDay.Equals(TEXT("Wednesday")))
			{
				return 3;
			}
			else if (WeekDay.Equals(TEXT("Thursday")))
			{
				return 4;
			}
			else if (WeekDay.Equals(TEXT("Friday")))
			{
				return 5;
			}
			else if (WeekDay.Equals(TEXT("Saturday")))
			{
				return 6;
			}
			else if (WeekDay.Equals(TEXT("Sunday")))
			{
				return 7;
			}
		}

		return -1;
	};

	auto ParseMonth = [](const FString& Month) -> int32
	{
		const int32 NumChars = Month.Len();

		if (NumChars == 3)
		{
			if (Month.Equals(TEXT("Jan")))
			{
				return 1;
			}
			else if (Month.Equals(TEXT("Feb")))
			{
				return 2;
			}
			else if (Month.Equals(TEXT("Mar")))
			{
				return 3;
			}
			else if (Month.Equals(TEXT("Apr")))
			{
				return 4;
			}
			else if (Month.Equals(TEXT("May")))
			{
				return 5;
			}
			else if (Month.Equals(TEXT("Jun")))
			{
				return 6;
			}
			else if (Month.Equals(TEXT("Jul")))
			{
				return 7;
			}
			else if (Month.Equals(TEXT("Aug")))
			{
				return 8;
			}
			else if (Month.Equals(TEXT("Sep")))
			{
				return 9;
			}
			else if (Month.Equals(TEXT("Oct")))
			{
				return 10;
			}
			else if (Month.Equals(TEXT("Nov")))
			{
				return 11;
			}
			else if (Month.Equals(TEXT("Dec")))
			{
				return 12;
			}
		}

		return -1;
	};

	auto ParseDate1 = [ParseMonth](const FString& DayStr, const FString& MonStr, const FString& YearStr, int32& Month, int32& Day, int32& Year) -> bool
	{
		// date1 = 2DIGIT SP month SP 4DIGIT
		// ; day month year(e.g., 02 Jun 1982)

		Day = FCString::Atoi(*DayStr);
		Month = ParseMonth(MonStr);
		Year = (YearStr.Len() == 4) ? FCString::Atoi(*YearStr) : -1;

		return (Day > 0 && Day <= 31) && (Month > 0 && Month <= 12) && (Year > 0 && Year <= 9999);
	};

	auto ParseDate2 = [ParseMonth](const FString& Date2, int32& Month, int32& Day, int32& Year) -> bool
	{
		// date2 = 2DIGIT "-" month "-" 2DIGIT
		// ; day - month - year(e.g., 02 - Jun - 82)
		TArray<FString> Tokens;

		// split up on a single delimiter
		int32 NumTokens = Date2.ParseIntoArray(Tokens, TEXT("-"), true);
		if (NumTokens == 3)
		{
			Day = FCString::Atoi(*Tokens[0]);
			Month = ParseMonth(Tokens[1]);
			Year = FCString::Atoi(*Tokens[2]);
			
			// Horrible assumption here, but this is a deprecated part of the spec
			Year += 1900;
		}

		return (Day > 0 && Day <= 31) && (Month > 0 && Month <= 12) && (Year > 0 && Year <= 9999);
	};

	auto ParseDate3 = [ParseMonth](const FString& MonStr, const FString& DayStr, int32& Month, int32& Day) -> bool
	{
		// date3 = month SP(2DIGIT | (SP 1DIGIT))
		// ; month day(e.g., Jun  2)
		const int32 NumDigits = DayStr.Len();
		Day = (NumDigits > 0 && NumDigits <= 2) ? FCString::Atoi(*DayStr) : -1;
		Month = ParseMonth(MonStr);

		return (Day > 0 && Day <= 31) && (Month > 0 && Month <= 12);
	};

	if (!HttpDate.IsEmpty())
	{
		TArray<FString> Tokens;

		// split up on a single delimiter
		int32 NumTokens = HttpDate.ParseIntoArray(Tokens, TEXT(" "), true);

		if (NumTokens > 0 && ensure(Tokens.Num() == NumTokens))
		{
			int32 Month = 0;
			int32 Day = 0;
			int32 Year = 0;
			int32 Hour = 0;
			int32 Minute = 0;
			int32 Second = 0;
			int32 Millisecond = 0;

			if (Tokens[0].EndsWith(TEXT(",")))
			{
				Tokens[0].RemoveAt(Tokens[0].Len() - 1, 1);
			}

			if (Tokens[Tokens.Num() - 1].Equals(TEXT("GMT")))
			{
				// rfc1123 - date | rfc850 - date 
				if (Tokens.Num() == 6)
				{
					int32 WkDay = ParseWkday(Tokens[0]);

					if (WkDay > 0)
					{
						// rfc1123 - date = wkday "," SP date1 SP time SP "GMT"
						if (ParseDate1(Tokens[1], Tokens[2], Tokens[3], Month, Day, Year))
						{
							ParseTime(Tokens[4], Hour, Minute, Second);
						}
					}
				}
				else if (Tokens.Num() == 4)
				{
					// rfc850 - date = weekday "," SP date2 SP time SP "GMT"
					int32 WeekDay = ParseWeekday(Tokens[0]);

					if (WeekDay > 0)
					{
						if (ParseDate2(Tokens[1], Month, Day, Year))
						{
							ParseTime(Tokens[2], Hour, Minute, Second);
						}
					}
				}

			}
			else if (Tokens.Num() == 5)
			{
				// asctime - date = wkday SP date3 SP time SP 4DIGIT
				int32 WkDay = ParseWkday(Tokens[0]);

				if (WkDay > 0)
				{
					if (ParseDate3(Tokens[1], Tokens[2], Month, Day))
					{
						if (ParseTime(Tokens[3], Hour, Minute, Second))
						{
							if (Tokens[4].Len() == 4)
							{
								Year = FCString::Atoi(*Tokens[4]);
							}
						}
					}
				}
			}

			if (Validate(Year, Month, Day, Hour, Minute, Second, Millisecond))
			{
				// convert the tokens to numbers
				OutDateTime = FDateTime(Year, Month, Day, Hour, Minute, Second, Millisecond);

				return true;
			}
		}
	}

	return false;
}


bool FDateTime::ParseIso8601(const TCHAR* DateTimeString, FDateTime& OutDateTime)
{
	// DateOnly: YYYY-MM-DD
	// DateTime: YYYY-mm-ddTHH:MM:SS(.sss)(Z|+hh:mm|+hhmm|-hh:mm|-hhmm)

	const TCHAR* Ptr = DateTimeString;
	TCHAR* Next = nullptr;

	int32 Year = 0, Month = 0, Day = 0;
	int32 Hour = 0, Minute = 0, Second = 0, Millisecond = 0;
	int32 TzHour = 0, TzMinute = 0;

	// get date
	Year = FCString::Strtoi(Ptr, &Next, 10);

	if ((Next <= Ptr) || (*Next == TCHAR('\0')))
	{
		return false;
	}

	Ptr = Next + 1; // skip separator
	Month = FCString::Strtoi(Ptr, &Next, 10);

	if ((Next <= Ptr) || (*Next == TCHAR('\0')))
	{
		return false;
	}

	Ptr = Next + 1; // skip separator
	Day = FCString::Strtoi(Ptr, &Next, 10);

	if (Next <= Ptr)
	{
		return false;
	}

	// check whether this is date and time
	if (*Next == TCHAR('T'))
	{
		Ptr = Next + 1;

		// parse time
		Hour = FCString::Strtoi(Ptr, &Next, 10);

		if ((Next <= Ptr) || (*Next == TCHAR('\0')))
		{
			return false;
		}

		Ptr = Next + 1; // skip separator
		Minute = FCString::Strtoi(Ptr, &Next, 10);

		if ((Next <= Ptr) || (*Next == TCHAR('\0')))
		{
			return false;
		}

		Ptr = Next + 1; // skip separator
		Second = FCString::Strtoi(Ptr, &Next, 10);

		if (Next <= Ptr)
		{
			return false;
		}

		// check for milliseconds
		if (*Next == TCHAR('.'))
		{
			Ptr = Next + 1;

			int64 MillisecondTemp = FCString::Strtoi64(Ptr, &Next, 10);
			

			//We support up to 18 digits to avoid rounding issue with 19 digits
			if ((Next <= Ptr) || (Next > Ptr + 18))
			{
				return false;
			}

			int32 Digits = UE_PTRDIFF_TO_INT32(Next - Ptr);

			if(Digits < 3)
			{
				//multiplying to account for the missing digits (would be zeros), ie 2020-08-24T05:56:14.4 should result in 400ms
				for (; Digits < 3; ++Digits)
				{
					MillisecondTemp *= 10;
				}
			}
			else if (Digits > 3)
			{
				//converting to milliseconds with rounding up -> 2020-08-24T05:56:14.459826919 will result in 460ms.
				int64 Divisor = 1;
				for (; Digits > 3; --Digits)
				{
					Divisor *= 10;
				}
				
				MillisecondTemp = (MillisecondTemp + (Divisor >> 1 )) / Divisor;
			}
			
			Millisecond = FMath::Min(static_cast<int32>(MillisecondTemp), 999);
		}

		// see if the timezone offset is included
		if (*Next == TCHAR('+') || *Next == TCHAR('-'))
		{
			// include the separator since it's + or -
			Ptr = Next;

			// parse the timezone offset
			TzHour = FCString::Strtoi(Ptr, &Next, 10);

			if (Next - Ptr == 3) // for "+/-hh:mm" and "+/-hh" cases
			{
				if (*Next != TCHAR('\0')) // "+/-hh:mm"
				{
					if (*Next != TCHAR(':'))
					{
						return false;
					}
					Ptr = Next + 1; // skip colon
					TzMinute = FCString::Strtoi(Ptr, &Next, 10);
				}
			}
			else if (Next - Ptr == 5) // for "+/-hhmm" case
			{
				TzMinute = TzHour % 100;
				TzHour /= 100;
			}
			else
			{
				return false;
			}
		}
		else if ((*Next != TCHAR('\0')) && (*Next != TCHAR('Z')))
		{
			return false;
		}
	}
	else if (*Next != TCHAR('\0'))
	{
		return false;
	}

	if (!Validate(Year, Month, Day, Hour, Minute, Second, Millisecond))
	{
		return false;
	}

	FDateTime Final(Year, Month, Day, Hour, Minute, Second, Millisecond);

	// adjust for the timezone (bringing the DateTime into UTC)
	int32 TzOffsetMinutes = (TzHour < 0) ? TzHour * 60 - TzMinute : TzHour * 60 + TzMinute;
	Final -= FTimespan::FromMinutes(TzOffsetMinutes);
	OutDateTime = Final;

	return true;
}

static FDateTime PlatformUtcNow()
{
	int32 Year, Month, Day, DayOfWeek;
	int32 Hour, Minute, Second, Millisecond;

	FPlatformTime::UtcTime(Year, Month, DayOfWeek, Day, Hour, Minute, Second, Millisecond);

	return FDateTime(Year, Month, Day, Hour, Minute, Second, Millisecond);
}

#if USE_ESTIMATED_UTCNOW

static void RebaseEstimatedUtcNow()
{
	FPlatformAtomics::AtomicStore(&RebaseEstimatedUtcNowViaDelegate, true);
}

static FDateTime EstimatedUtcNow()
{
	static volatile int32 bInitialized = false;
	static volatile int32 InitializationGate = false;
	static volatile int32 FastPathCounter = 0;

	static FDateTime BaseUtc;
	static uint64 BaseCycleCounter;

	// track if any threads are potentially on the 'fast path'
	FPlatformAtomics::InterlockedIncrement(&FastPathCounter);

	if (!FPlatformAtomics::AtomicRead(&bInitialized))
	{
		// 'slow' (initialization) path
		FPlatformAtomics::InterlockedDecrement(&FastPathCounter);

		// ensure only one thread can enter the initialization branch and no threads are on the 'fast path'
		const int32_t OriginalValue = FPlatformAtomics::InterlockedCompareExchange(&InitializationGate, true, false);
		const bool bCanInitialize = InitializationGate != OriginalValue && FastPathCounter == 0;

		if (bCanInitialize)
		{
			BaseUtc = PlatformUtcNow();
			BaseCycleCounter = FPlatformTime::Cycles64();

			static FDelegateHandle AppResumeDelegateHandle;
			static FDelegateHandle AppReactivatedDelegateHandle;

			if (!AppResumeDelegateHandle.IsValid())
			{
				AppResumeDelegateHandle = FCoreDelegates::ApplicationHasEnteredForegroundDelegate.AddStatic(&RebaseEstimatedUtcNow);
			}
			if (!AppReactivatedDelegateHandle.IsValid())
			{
				AppReactivatedDelegateHandle = FCoreDelegates::ApplicationHasEnteredForegroundDelegate.AddStatic(&RebaseEstimatedUtcNow);
			}

			FPlatformAtomics::AtomicStore(&bInitialized, true);
			FPlatformAtomics::AtomicStore(&RebaseEstimatedUtcNowViaDelegate, false);
			return BaseUtc;
		}
		else
		{
			return PlatformUtcNow();
		}
	}
	else
	{
		// 'fast' path 
		uint64_t CyclesElapsed = FPlatformTime::Cycles64() - BaseCycleCounter;
		FTimespan SecondsElapsedSinceInit = FTimespan::FromSeconds(FPlatformTime::ToSeconds64(CyclesElapsed));
		FDateTime CurrentUtc = BaseUtc + SecondsElapsedSinceInit;

		// UTC is (almost) free from complicating concerns that interrupt the consistent increasing
		// of time, such as daylight savings.  However leap seconds can still be inserted at the end
		// of the last day of the month (https://en.wikipedia.org/wiki/Leap_second)
		//
		// Check to see if we are crossing the end-of-month boundary, and reset initialization if so.  
		// If the UTC provider takes leap seconds then the next time through this function BaseUtc will 
		// be initialized with the leap second added
		int32 BaseDay;
		BaseDay = BaseUtc.GetDay();

		int32 CurrentDay;
		CurrentDay = CurrentUtc.GetDay();

		static int32 SecondsBeforeRebase = FMath::Max<int32>(0, CVarEstimatedUtcNowRebaseTimeSeconds.GetValueOnAnyThread());
		static FTimespan SecondsBeforeRebaseTimespan = FTimespan::FromSeconds(SecondsBeforeRebase);

		const bool CheckForLeapSecondRebase = (CurrentDay == 1 && BaseDay != 1);
		const bool TimeElapsedRebase = SecondsElapsedSinceInit > SecondsBeforeRebaseTimespan;
		if (CheckForLeapSecondRebase || TimeElapsedRebase || FPlatformAtomics::AtomicRead(&RebaseEstimatedUtcNowViaDelegate))
		{
			FPlatformAtomics::AtomicStore(&InitializationGate, false);
			FPlatformAtomics::AtomicStore(&bInitialized, false);

			FPlatformAtomics::InterlockedDecrement(&FastPathCounter);
			return PlatformUtcNow();
		}

		FPlatformAtomics::InterlockedDecrement(&FastPathCounter);
		return CurrentUtc;
	}
}
#endif // #if USE_ESTIMATED_UTCNOW

FDateTime FDateTime::UtcNow()
{
#if USE_ESTIMATED_UTCNOW
	return EstimatedUtcNow();
#else
	return PlatformUtcNow();
#endif
}

bool FDateTime::Validate(int32 Year, int32 Month, int32 Day, int32 Hour, int32 Minute, int32 Second, int32 Millisecond)
{
	return (Year >= 1) && (Year <= 9999) &&
		(Month >= 1) && (Month <= 12) &&
		(Day >= 1) && (Day <= DaysInMonth(Year, Month)) &&
		(Hour >= 0) && (Hour <= 23) &&
		(Minute >= 0) && (Minute <= 59) &&
		(Second >= 0) && (Second <= 59) &&
		(Millisecond >= 0) && (Millisecond <= 999);
}

