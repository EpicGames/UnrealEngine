// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/DefaultValueHelper.h"

#include "Containers/Array.h"
#include "Math/Color.h"
#include "Math/Rotator.h"
#include "Math/Vector.h"
#include "Math/Vector2D.h"
#include "Math/Vector4.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CString.h"
#include "Misc/Char.h"
#include "Misc/Parse.h"

bool FDefaultValueHelper::Is(const FString& Source, const TCHAR* CompareStr)
{
	check(NULL != CompareStr);

	int32 Pos = 0;
	if( !Trim(Pos, Source) )
	{
		return false;
	}

	if( Source.Find(CompareStr, ESearchCase::CaseSensitive) != Pos )
	{
		return false;
	}
	Pos += FCString::Strlen(CompareStr);

	if( Trim(Pos, Source) )
	{
		return false;
	}

	return true;
}


FString FDefaultValueHelper::RemoveWhitespaces(const FString& Source)
{
	FString Result;
	if( !Source.IsEmpty() )
	{
		Result.Reserve( Source.Len() );
		for(int Pos = 0; Pos < Source.Len(); ++Pos)
		{
			if( !IsWhitespace( Source[Pos] ) )
			{
				Result.AppendChar( Source[Pos] );
			}
		}
	}
	return Result;
}


const TCHAR* FDefaultValueHelper::EndOf(const FString& Source)
{
	check(!Source.IsEmpty());
	return *Source + Source.Len();
}


const TCHAR* FDefaultValueHelper::StartOf(const FString& Source)
{
	check(!Source.IsEmpty());
	return *Source;
}


TCHAR FDefaultValueHelper::TS(const TCHAR* Sign)
{
	check(NULL != Sign);
	check(0 != Sign[0]);
	return Sign[0];
}


bool FDefaultValueHelper::IsWhitespace(TCHAR Char)
{
	return FChar::IsWhitespace(Char)
		|| FChar::IsLinebreak(Char)
		|| ( TS(TEXT("\r")) == Char );
}


bool FDefaultValueHelper::Trim(int32& Pos, const FString& Source)
{
	for(; Pos < Source.Len() && IsWhitespace(Source[Pos]); ++Pos) { }
	return (Pos < Source.Len());
}


bool FDefaultValueHelper::Trim(const TCHAR* & Start, const TCHAR* End )
{
	check(NULL != Start);
	check(NULL != End);
	for(; ( Start < End ) && IsWhitespace( *Start ); ++Start ) { }
	return Start < End;
}


FString FDefaultValueHelper::GetUnqualifiedEnumValue(const FString& Source)
{
	auto SeparatorPosition = Source.Find(FString("::"), ESearchCase::CaseSensitive);
	if (SeparatorPosition == INDEX_NONE)
	{
		return Source;
	}

	return Source.RightChop(SeparatorPosition + 2);
}


bool FDefaultValueHelper::HasWhitespaces(const FString& Source)
{
	for(int Pos = 0; Pos < Source.Len(); ++Pos)
	{
		if( IsWhitespace( Source[Pos] ) )
		{
			return true;
		}
	}
	return false;
}


bool FDefaultValueHelper::GetParameters(const FString& Source, const FString& TypeName, FString& OutForm)
{
	int32 Pos = 0;

	if( !Trim(Pos, Source) )
	{
		return false;
	}

	// find the beginning of actual val after "TypeName ( "
	if( Source.Find(TypeName, ESearchCase::CaseSensitive) != Pos )
	{
		return false;
	}
	Pos += TypeName.Len();
	if( !Trim(Pos, Source) )
	{
		return false;
	}
	if( TS(TEXT("(")) != Source[Pos++] )
	{
		return false;
	}
	if( !Trim(Pos, Source) )
	{
		return false;
	}
	const int32 StartPos = Pos;

	int32 EndPos = -1, PendingParentheses = 1;
	// find the end of the actual string before " ) "
	for(Pos = Source.Len() - 1; Pos >= StartPos; --Pos )
	{
		if( TS(TEXT(")")) == Source[Pos]) 
		{
			if (--PendingParentheses == 0 && Pos == StartPos)
			{
				// Empty param list
				EndPos = StartPos;
			}
		}
		else if(!IsWhitespace(Source[Pos]))
		{
			EndPos = Pos+1;
			break;
		}
	}

	if(EndPos < 0 || 0 != PendingParentheses)
	{
		return false;
	}

	OutForm = Source.Mid(StartPos, EndPos - StartPos);
	return true;
}


////////////////////////////////////////////////////////

bool FDefaultValueHelper::IsStringValidInteger(const TCHAR* Start, const TCHAR* End, int32& OutBase)
{
	bool bAllowHex = false;
	bool bADigitFound = false;

	if( !Trim(Start, End) )
	{
		return false;
	}

	if( ( TS(TEXT("-")) == *Start ) || ( TS(TEXT("+")) == *Start ) )
	{
		Start++;
	}

	if( !Trim(Start, End) )
	{
		return false;
	}

	if( TS(TEXT("0")) == *Start )
	{
		Start++;
		bADigitFound = true;
		if( ( TS(TEXT("x")) == *Start ) || ( TS(TEXT("X")) == *Start ) )
		{
			Start++;
			bAllowHex = true;
			bADigitFound = false;
			OutBase = 16;
		}
		else
		{
			OutBase = 8;
		}
	}
	else
	{
		OutBase = 10;
	}

	if( bAllowHex )
	{
		for(; ( Start < End ) && FChar::IsHexDigit( *Start ); ++Start)
		{
			bADigitFound = true;
		}
	}
	else
	{
		for(; ( Start < End ) && FChar::IsDigit( *Start ); ++Start)
		{
			bADigitFound = true;
		}
	}

	return ( !Trim(Start, End) ) && bADigitFound;
}


bool FDefaultValueHelper::IsStringValidInteger(const TCHAR* Start, const TCHAR* End)
{
	int32 Base;
	return IsStringValidInteger(Start, End, /*out*/ Base);
}
	
	
bool FDefaultValueHelper::IsStringValidInteger(const FString& Source)
{
	if(!Source.IsEmpty())
	{
		int32 Base;
		return IsStringValidInteger(StartOf(Source), EndOf(Source), Base);
	}
	return false;
}


bool FDefaultValueHelper::IsStringValidFloat(const TCHAR* Start, const TCHAR* End)
{
	if( !Trim(Start, End) )
	{
		return false;
	}

	if( ( TS(TEXT("-")) == *Start ) || ( TS(TEXT("+")) == *Start ) )
	{
		Start++;
	}
	if( !Trim(Start, End) )
	{
		return false;
	}

	for(; ( Start < End ) && FChar::IsDigit( *Start ); ++Start ) { }
	if( TS(TEXT(".")) == *Start )
	{
		Start++;
	}

	for(; ( Start < End ) && FChar::IsDigit( *Start ); ++Start ) { }

	if( ( TS(TEXT("e")) == *Start ) || ( TS(TEXT("E")) == *Start ) )
	{
		Start++;
		if( ( TS(TEXT("-")) == *Start ) || ( TS(TEXT("+")) == *Start ) )
		{
			Start++;
		}
	}
	for(; ( Start < End ) && FChar::IsDigit( *Start ); ++Start) { }
	if( ( TS(TEXT("f")) == *Start ) || ( TS(TEXT("F")) == *Start ) )
	{
		Start++;
	}

	return !Trim(Start, End);
}


bool FDefaultValueHelper::IsStringValidFloat(const FString& Source)
{
	if(!Source.IsEmpty())
	{
		return IsStringValidFloat( StartOf(Source), EndOf(Source) );
	}
	return false;
}


bool FDefaultValueHelper::IsStringValidVector(const FString& Source)
{
	FVector TempVector;
	return FDefaultValueHelper::ParseVector(Source, TempVector);
}


bool FDefaultValueHelper::IsStringValidRotator(const FString& Source)
{
	return IsStringValidVector(Source);
}


bool FDefaultValueHelper::IsStringValidLinearColor(const FString& Source)
{
	FLinearColor TempColor;
	return ParseLinearColor(Source, TempColor);
}


bool FDefaultValueHelper::StringFromCppString(const FString& Source, const FString& TypeName, FString& OutForm)
{
	int32 Pos = 0, PendingParentheses = 0;

	if( !Trim(Pos, Source) )
	{
		return false;
	}

	// remove "TypeName ( "
	if (Source.Find(TypeName, ESearchCase::CaseSensitive) == Pos)
	{
		Pos += TypeName.Len();
		if( !Trim(Pos, Source) )
		{
			return false;
		}

		if( TS(TEXT("(")) != Source[Pos++] )
		{
			return false;
		}
		PendingParentheses++;
		if( !Trim(Pos, Source) )
		{
			return false;
		}		
	}

	// remove "TEXT ( "
	const TCHAR* TextStr = TEXT("TEXT");
	if (Source.Find(TextStr, ESearchCase::CaseSensitive) == Pos)
	{
		Pos += FCString::Strlen(TextStr);
		if( !Trim(Pos, Source) )
		{
			return false;
		}
		if( TS(TEXT("(")) != Source[Pos++] )
		{
			return false;
		}
		PendingParentheses++;
		if( !Trim(Pos, Source) )
		{
			return false;
		}
	}

	// ensure the beginning of actual string is found
	if( TS(TEXT("\"")) != Source[Pos++] )
	{
		return false;
	}
	const int32 StartPos = Pos;
	int32 EndPos = -1;

	// find end of the actual string
	for(; Pos < Source.Len(); ++Pos)
	{
		if( ( TS(TEXT("\"")) == Source[Pos] ) && ( TS(TEXT("\\")) != Source[Pos-1] ) )
		{
			EndPos = Pos;
			Pos++;
			break;
		}
	}

	for(; Pos < Source.Len(); ++Pos)
	{
		if( TS(TEXT(")")) == Source[Pos] ) 
		{
			PendingParentheses--;
		}
		else if(!IsWhitespace(Source[Pos]))
		{
			return false;
		}
	}

	if(EndPos < 0 || 0 != PendingParentheses)
	{
		return false;
	}

	OutForm = Source.Mid(StartPos, EndPos - StartPos);
	return true;
}


////////////////////////////////////////////////////////

bool FDefaultValueHelper::ParseVector(const FString& Source, FVector3f& OutVal)
{
	// LWC_TODO: Perf pessimization, especially if LWC is disabled!
	FVector3d TempVec;
	if (ParseVector(Source, TempVec))
	{
		OutVal = FVector3f(TempVec);
		return true;
	}
	return false;
}

bool FDefaultValueHelper::ParseVector(const FString& Source, FVector3d& OutVal)
{
	const bool bHasWhitespace = HasWhitespaces(Source);
	const FString NoWhitespace = bHasWhitespace ? RemoveWhitespaces(Source) : FString();
	const FString& ProperSource = bHasWhitespace ? NoWhitespace : Source;
	if(ProperSource.IsEmpty())
	{
		return false;
	}

	const TCHAR* Start = StartOf(ProperSource);
	const TCHAR* FirstComma = FCString::Strstr( Start, TEXT(",") );
	if(!FirstComma)
	{
		return false;
	}

	const TCHAR* SecondComma = FCString::Strstr( FirstComma + 1,  TEXT(",") );
	if(!SecondComma)
	{
		return false;
	}

	// there must be exactly 2 commas in the string
	if( FCString::Strstr( SecondComma + 1, TEXT(",") ) )
	{
		return false;
	}

	const TCHAR* End = EndOf(ProperSource);
	if(	!IsStringValidFloat( Start, FirstComma ) ||
		!IsStringValidFloat( FirstComma + 1, SecondComma ) ||
		!IsStringValidFloat( SecondComma + 1, End ) )
	{
		// Fallback to x= format
		return OutVal.InitFromString(Source);
	}
	
	OutVal = FVector3d( 
		FCString::Atod(Start),
		FCString::Atod(FirstComma + 1),
		FCString::Atod(SecondComma + 1) );
	return true;
}



bool FDefaultValueHelper::ParseVector2D(const FString& Source, FVector2f& OutVal)
{
	// LWC_TODO: Perf pessimization, especially if LWC is disabled!
	FVector2d TempVec;
	if (ParseVector2D(Source, TempVec))
	{
		OutVal = FVector2f(TempVec);
		return true;
	}
	return false;
}

bool FDefaultValueHelper::ParseVector2D(const FString& Source, FVector2d& OutVal)
{
	const bool bHasWhitespace = HasWhitespaces(Source);
	const FString NoWhitespace = bHasWhitespace ? RemoveWhitespaces(Source) : FString();
	const FString& ProperSource = bHasWhitespace ? NoWhitespace : Source;
	if(ProperSource.IsEmpty())
	{
		return false;
	}

	const TCHAR* Start = StartOf(ProperSource);
	const TCHAR* FirstComma = FCString::Strstr( Start, TEXT(",") );
	if(!FirstComma)
	{
		return false;
	}

	const TCHAR* End = EndOf(ProperSource);
	if(	!IsStringValidFloat( Start, FirstComma ) ||
		!IsStringValidFloat( FirstComma + 1, End ) )
	{
		// Fallback to x= format
		return OutVal.InitFromString(Source);
	}

	OutVal = FVector2d( 
		FCString::Atod(Start),
		FCString::Atod(FirstComma + 1) );
	return true;
}


bool FDefaultValueHelper::ParseVector4(const FString& Source, FVector4f& OutVal)
{
	// LWC_TODO: Perf pessimization, especially if LWC is disabled!
	FVector4d TempVec;
	if (ParseVector4(Source, TempVec))
	{
		OutVal = FVector4f(TempVec);
		return true;
	}
	return false;
}

bool FDefaultValueHelper::ParseVector4(const FString& Source, FVector4d& OutVal)
{
	double X, Y, Z, W;
	TArray<FString> SourceParts;

	if (Source.ParseIntoArray(SourceParts, TEXT(",")) == 4 &&
		ParseDouble(SourceParts[0], X) &&
		ParseDouble(SourceParts[1], Y) && 
		ParseDouble(SourceParts[2], Z) &&
		ParseDouble(SourceParts[3], W))
	{
		OutVal = FVector4d(X, Y, Z, W);
		return true;
	}

	// Fallback to x= format
	return OutVal.InitFromString(Source);
}


bool FDefaultValueHelper::ParseRotator(const FString& Source, FRotator3f& OutVal)
{
	// LWC_TODO: Perf pessimization, especially if LWC is disabled!
	FRotator3d TempRotator;
	if (ParseRotator(Source, TempRotator))
	{
		OutVal = FRotator3f(TempRotator);
		return true;
	}
	return false;
}

bool FDefaultValueHelper::ParseRotator(const FString& Source, FRotator3d& OutVal)
{
	FVector3d Vector;
	if (ParseVector(Source, Vector))
	{
		using FReal = decltype(FRotator3d::Pitch);
		OutVal = FRotator3d((FReal)Vector.X, (FReal)Vector.Y, (FReal)Vector.Z);
		return true;
	}

	// Fallback to p= format
	if (OutVal.InitFromString(Source))
	{
		return true;
	}

	// try Pitch= format
	FRotator3d Rotator = FRotator3d::ZeroRotator;
	if (FParse::Value(*Source, TEXT("Pitch="), Rotator.Pitch) &&
		FParse::Value(*Source, TEXT("Yaw="), Rotator.Yaw) &&
		FParse::Value(*Source, TEXT("Roll="), Rotator.Roll))
	{
		OutVal = Rotator;
		return true;
	}

	return false;
}


bool FDefaultValueHelper::ParseInt(const TCHAR* Source, int32& OutVal)
{
	int32 Base;
	const int32 Len = (Source && *Source) ? FCString::Strlen(Source) : 0;
	if( Len > 0 && IsStringValidInteger( Source, Source + Len, Base ) )
	{
		const bool bHasWhitespace = HasWhitespaces(Source);
		const FString NoWhitespace = bHasWhitespace ? RemoveWhitespaces(Source) : FString();
		OutVal = FCString::Strtoi( bHasWhitespace ? *NoWhitespace : Source , NULL, Base );
		return true;
	}
	return false;
}

bool FDefaultValueHelper::ParseBool(const FString& Source, bool& OutVal)
{
	const bool bHasWhitespace = HasWhitespaces(Source);
	const FString NoWhitespace = bHasWhitespace ? RemoveWhitespaces(Source) : FString();
	const FString& ProperSource = bHasWhitespace ? NoWhitespace : Source;
	if (ProperSource.IsEmpty())
	{
		return false;
	}

	const FString SourceToLower = ProperSource.ToLower();
	if (SourceToLower == TEXT("true"))
	{
		OutVal = true;
		return true;
	}
	
	if (SourceToLower == TEXT("false"))
	{
		OutVal = false;
		return true;
	}
	
	// If it's not in the "true/false" form, try to parse it as an integer
	int32 Base;
	if (!Source.IsEmpty() && IsStringValidInteger(StartOf(Source), EndOf(Source), Base))
	{
		int32 ValAsInteger = FCString::Strtoi(*ProperSource, NULL, Base);
		OutVal = static_cast<bool>(ValAsInteger);
		return true;
	}

	return false;
}

bool FDefaultValueHelper::ParseInt64(const FString& Source, int64& OutVal)
{
	int32 Base;
	if( !Source.IsEmpty() && IsStringValidInteger( StartOf(Source), EndOf(Source), Base ) )
	{
		const bool bHasWhitespace = HasWhitespaces(Source);
		const FString NoWhitespace = bHasWhitespace ? RemoveWhitespaces(Source) : FString();
		OutVal = FCString::Strtoui64( bHasWhitespace ? *NoWhitespace : *Source , NULL, Base );
		return true;
	}
	return false;
}


bool FDefaultValueHelper::ParseFloat(const FString& Source, float& OutVal)
{
	if( IsStringValidFloat( Source ) )
	{
		const bool bHasWhitespace = HasWhitespaces(Source);
		const FString NoWhitespace = bHasWhitespace ? RemoveWhitespaces(Source) : FString();
		OutVal =  FCString::Atof( bHasWhitespace ? *NoWhitespace : *Source );
		return true;
	}
	return false;
}


bool FDefaultValueHelper::ParseDouble(const FString& Source, double& OutVal)
{
	if( IsStringValidFloat( Source ) )
	{
		const bool bHasWhitespace = HasWhitespaces(Source);
		const FString NoWhitespace = bHasWhitespace ? RemoveWhitespaces(Source) : FString();
		OutVal =  FCString::Atod( bHasWhitespace ? *NoWhitespace : *Source );
		return true;
	}
	return false;
}


bool FDefaultValueHelper::ParseLinearColor(const FString& Source, FLinearColor& OutVal)
{
	const bool bHasWhitespace = HasWhitespaces(Source);
	const FString NoWhitespace = bHasWhitespace ? RemoveWhitespaces(Source) : FString();
	const FString& ProperSource = bHasWhitespace ? NoWhitespace : Source;
	if(ProperSource.IsEmpty())
	{
		return false;
	}

	const TCHAR* Start = StartOf(ProperSource);
	const TCHAR* FirstComma = FCString::Strstr( Start, TEXT(",") );
	if(!FirstComma)
	{
		return false;
	}

	const TCHAR* SecondComma = FCString::Strstr( FirstComma + 1,  TEXT(",") );
	if(!SecondComma)
	{
		return false;
	}

	const TCHAR* ThirdComma = FCString::Strstr( SecondComma + 1,  TEXT(",") );
	const TCHAR* End = EndOf(ProperSource);
	if( ( NULL != ThirdComma ) && !IsStringValidFloat( ThirdComma + 1, End ) )
	{
		return false;
	}
	
	if(	!IsStringValidFloat( Start, FirstComma ) ||
		!IsStringValidFloat( FirstComma + 1, SecondComma ) ||
		!IsStringValidFloat( SecondComma + 1, ( NULL != ThirdComma ) ? ThirdComma : End ) )
	{
		// Fallback to x= format
		return OutVal.InitFromString(Source);
	}

	const float Alpha = ( NULL != ThirdComma ) ? FCString::Atof( ThirdComma + 1 ) : 1.0f;
	OutVal = FLinearColor( 
		FCString::Atof( Start ), 
		FCString::Atof( FirstComma + 1 ), 
		FCString::Atof( SecondComma + 1 ),
		Alpha );
	
	return true;
}

bool FDefaultValueHelper::ParseColor(const FString& Source, FColor& OutVal)
{
	const bool bHasWhitespace = HasWhitespaces(Source);
	const FString NoWhitespace = bHasWhitespace ? RemoveWhitespaces(Source) : FString();
	const FString& ProperSource = bHasWhitespace ? NoWhitespace : Source;
	if(ProperSource.IsEmpty())
	{
		return false;
	}

	const TCHAR* Start = StartOf(ProperSource);
	const TCHAR* FirstComma = FCString::Strstr( Start, TEXT(",") );
	if(!FirstComma)
	{
		return false;
	}

	const TCHAR* SecondComma = FCString::Strstr( FirstComma + 1,  TEXT(",") );
	if(!SecondComma)
	{
		return false;
	}

	const TCHAR* ThirdComma = FCString::Strstr( SecondComma + 1,  TEXT(",") );
	const TCHAR* End = EndOf(ProperSource);
	if( ( NULL != ThirdComma ) && !IsStringValidInteger( ThirdComma + 1, End ) )
	{
		return false;
	}
	
	if(	!IsStringValidInteger( Start, FirstComma ) ||
		!IsStringValidInteger( FirstComma + 1, SecondComma ) ||
		!IsStringValidInteger( SecondComma + 1, ( NULL != ThirdComma ) ? ThirdComma : End ) )
	{
		// Fallback to x= format
		return OutVal.InitFromString(Source);
	}

	const uint8 Alpha = ( NULL != ThirdComma ) ? (uint8)FCString::Atoi( ThirdComma + 1 ) : 255;
	OutVal = FColor( 
		(uint8)FCString::Atoi( Start ), 
		(uint8)FCString::Atoi( FirstComma + 1 ),
		(uint8)FCString::Atoi( SecondComma + 1 ),
		Alpha );
	
	return true;
}
