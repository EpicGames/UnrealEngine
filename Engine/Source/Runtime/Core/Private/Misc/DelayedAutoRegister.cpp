// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/DelayedAutoRegister.h"

#include "Containers/Set.h"
#include "Delegates/Delegate.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/AssertionMacros.h"
#include "Modules/ModuleManager.h"
#include "Templates/Function.h"

DECLARE_MULTICAST_DELEGATE_OneParam(FDelayedAutoRegisterDelegate, EDelayedRegisterRunPhase);

static TSet<EDelayedRegisterRunPhase> GPhasesAlreadyRun;

static FDelayedAutoRegisterDelegate& GetDelayedAutoRegisterDelegate(EDelayedRegisterRunPhase Phase)
{
	static FDelayedAutoRegisterDelegate Singleton[(uint8)EDelayedRegisterRunPhase::NumPhases];
	return Singleton[(uint8)Phase];
}

FDelayedAutoRegisterHelper::FDelayedAutoRegisterHelper(EDelayedRegisterRunPhase Phase, TFunction<void()> RegistrationFunction, const bool bRerunOnLiveCodingReload)
	: FDelayedAutoRegisterHelper(
		Phase,
		[RegistrationFunction](const EDelayedRegisterRunPhase) { RegistrationFunction(); },
		bRerunOnLiveCodingReload)
{
}

FDelayedAutoRegisterHelper::FDelayedAutoRegisterHelper(
	EDelayedRegisterRunPhase Phase,
	TFunction<void(const EDelayedRegisterRunPhase)> RegistrationFunction,
	const bool bRerunOnLiveCodingReload)
{
#if WITH_EDITOR && WITH_LIVE_CODING
	// We can use both the provided phase AND LiveCodingReload
	if (bRerunOnLiveCodingReload && Phase != EDelayedRegisterRunPhase::LiveCodingReload)
	{
		GetDelayedAutoRegisterDelegate(EDelayedRegisterRunPhase::LiveCodingReload).AddLambda(RegistrationFunction);
	}
#endif

	// if the phase has already passed, we just run the function immediately
	if (Phase < EDelayedRegisterRunPhase::NumRunOncePhases && GPhasesAlreadyRun.Contains(Phase))
	{
		RegistrationFunction(Phase);
	}
	else
	{
		GetDelayedAutoRegisterDelegate(Phase).AddLambda(RegistrationFunction);
	}
}

namespace
{
	const TCHAR* LexToString(EDelayedRegisterRunPhase Phase)
	{
		switch (Phase)
		{
		case EDelayedRegisterRunPhase::StartOfEnginePreInit: return TEXT("StartOfEnginePreInit");
		case EDelayedRegisterRunPhase::FileSystemReady: return TEXT("FileSystemReady");
		case EDelayedRegisterRunPhase::TaskGraphSystemReady: return TEXT("TaskGraphSystemReady");
		case EDelayedRegisterRunPhase::StatSystemReady: return TEXT("StatSystemReady");
		case EDelayedRegisterRunPhase::IniSystemReady: return TEXT("IniSystemReady");
		case EDelayedRegisterRunPhase::EarliestPossiblePluginsLoaded: return TEXT("EarliestPossiblePluginsLoaded");
		case EDelayedRegisterRunPhase::PreRHIInit: return TEXT("PreRHIInit");
		case EDelayedRegisterRunPhase::ShaderTypesReady: return TEXT("ShaderTypesReady");
		case EDelayedRegisterRunPhase::PreObjectSystemReady: return TEXT("PreObjectSystemReady");
		case EDelayedRegisterRunPhase::ObjectSystemReady: return TEXT("ObjectSystemReady");
		case EDelayedRegisterRunPhase::DeviceProfileManagerReady: return TEXT("DeviceProfileManagerReady");
		case EDelayedRegisterRunPhase::EndOfEngineInit: return TEXT("EndOfEngineInit");
		case EDelayedRegisterRunPhase::LiveCodingReload: return TEXT("LiveCodingReload");

		case EDelayedRegisterRunPhase::NumRunOncePhases:
		case EDelayedRegisterRunPhase::NumPhases: return TEXT("UnprintableValue");

		default: checkf(false, TEXT("Add Phase to LexToString(EDelayedRegisterRunPhase) [int value: %d]"), (int)Phase); break;
		}

		return TEXT("Unknown");
	}
}

void FDelayedAutoRegisterHelper::RunAndClearDelayedAutoRegisterDelegates(EDelayedRegisterRunPhase RunPhase)
{
	const bool bIsRunOncePhase = RunPhase < EDelayedRegisterRunPhase::NumRunOncePhases;
	if (bIsRunOncePhase)
	{
		checkf(!GPhasesAlreadyRun.Contains(RunPhase), TEXT("Delayed Startup phase %s has already run - it is not expected to be run again!"), LexToString(RunPhase));
	}

	UE_LOGF(LogInit, Log, "Running DelayedAutoRegister Phase %ls", LexToString(RunPhase));

	// look in .ini for modules to load at this phase (only after ini system is ready!)
	if (RunPhase >= EDelayedRegisterRunPhase::IniSystemReady)
	{
		EDelayedRegisterRunPhase FirstPhase = RunPhase;
		EDelayedRegisterRunPhase LastPhase = RunPhase;

		// this phase needs to catch up
		if (RunPhase == EDelayedRegisterRunPhase::IniSystemReady)
		{
			FirstPhase = EDelayedRegisterRunPhase::StartOfEnginePreInit;
		}

		// Lambda to load modules from a specific config property
		auto LoadModulesFromConfigProperty = [RunPhase](EDelayedRegisterRunPhase IniPhase, const TCHAR* ConfigPropertyName)
		{
			TArray<FString> ModuleNames;
			GConfig->GetArray(ConfigPropertyName, LexToString(IniPhase), ModuleNames, GEngineIni);
			for (const FString& ModuleName : ModuleNames)
			{
				UE_LOGF(LogInit, Verbose, "Auto-loading module %ls during startup phase %ls", *ModuleName, LexToString(RunPhase));
				if (FModuleManager::Get().LoadModule(*ModuleName) == nullptr)
				{
					UE_LOGF(LogInit, Log, "Failed to auto-load module %ls during startup phase %ls", *ModuleName, LexToString(RunPhase));
				}
			}
		};

		for (EDelayedRegisterRunPhase IniPhase = FirstPhase; IniPhase <= LastPhase; IniPhase = (EDelayedRegisterRunPhase)((int)IniPhase + 1))
		{
			// Load modules from the standard config property
			LoadModulesFromConfigProperty(IniPhase, TEXT("AutoLoadModulesByPhase"));

			// Load editor-specific modules when running in editor
			if (GIsEditor)
			{
				LoadModulesFromConfigProperty(IniPhase, TEXT("AutoLoadModulesByPhase_Editor"));
			}
		}
	}


	// run all the delayed functions!
	GetDelayedAutoRegisterDelegate(RunPhase).Broadcast(RunPhase);

	if (bIsRunOncePhase)
	{
		GetDelayedAutoRegisterDelegate(RunPhase).Clear();
		GPhasesAlreadyRun.Add(RunPhase);
	}
}
