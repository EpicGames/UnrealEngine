// Copyright Epic Games, Inc. All Rights Reserved.

// Core includes.
#include "Misc/EmbeddedCommunication.h"
#include "Misc/ScopeLock.h"
#include "HAL/Event.h"
#include "HAL/PlatformProcess.h"
#include "Containers/Queue.h"
#include "Containers/Ticker.h"
#include "HAL/PlatformTime.h"
#include "HAL/ThreadManager.h"
#include "Misc/Parse.h"
#include "Misc/CommandLine.h"
#include "CoreGlobals.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Misc/FeedbackContext.h"
#include "Stats/Stats.h"

TMap<FName, FEmbeddedDelegates::FEmbeddedCommunicationParamsDelegate> FEmbeddedDelegates::NativeToEmbeddedDelegateMap;
TMap<FName, FEmbeddedDelegates::FEmbeddedCommunicationParamsDelegate> FEmbeddedDelegates::EmbeddedToNativeDelegateMap;
FSimpleMulticastDelegate FEmbeddedDelegates::SleepTickDelegate;
FCriticalSection FEmbeddedDelegates::NamedObjectRegistryLock;
TMap<FString, void*> FEmbeddedDelegates::NamedObjectRegistry;

FEmbeddedDelegates::FEmbeddedCommunicationParamsDelegate& FEmbeddedDelegates::GetNativeToEmbeddedParamsDelegateForSubsystem(FName SubsystemName)
{
	return NativeToEmbeddedDelegateMap.FindOrAdd(SubsystemName);
}

// calling out from engine to native wrapper
FEmbeddedDelegates::FEmbeddedCommunicationParamsDelegate& FEmbeddedDelegates::GetEmbeddedToNativeParamsDelegateForSubsystem(FName SubsystemName)
{
	return EmbeddedToNativeDelegateMap.FindOrAdd(SubsystemName);
}

bool FEmbeddedDelegates::IsEmbeddedSubsystemAvailable(FName SubsystemName)
{
	FEmbeddedCommunicationParamsDelegate* NativeToEmbeddedDelegate = NativeToEmbeddedDelegateMap.Find(SubsystemName);

	return NativeToEmbeddedDelegate && NativeToEmbeddedDelegate->IsBound();
}

void FEmbeddedDelegates::SetNamedObject(const FString& Name, void* Object)
{
	FScopeLock Lock(&NamedObjectRegistryLock);
	NamedObjectRegistry.Add(Name, Object);
}
void* FEmbeddedDelegates::GetNamedObject(const FString& Name)
{
	FScopeLock Lock(&NamedObjectRegistryLock);
	return NamedObjectRegistry.FindRef(Name);
}


#if BUILD_EMBEDDED_APP

static FEvent* GSleepEvent = nullptr;
static TAtomic<int32> GTickWithoutSleepCount(0);
static FCriticalSection GEmbeddedLock;
//static TArray<TFunction<void()>> GEmbeddedQueues[5];
static TQueue<TFunction<void()>> GEmbeddedQueues[5];

static TMap<FName, int> GRenderingWakeMap;
static TMap<FName, int> GTickWakeMap;

static bool HasMessagesInQueue()
{
	FScopeLock Lock(&GEmbeddedLock);

	for (auto& It : GEmbeddedQueues)
	{
		if (!It.IsEmpty())
		{
			return true;
		}
	}
	return false;
}
#endif

void FEmbeddedCommunication::Init()
{
#if BUILD_EMBEDDED_APP
 	FScopeLock Lock(&GEmbeddedLock);
 	GSleepEvent = FPlatformProcess::GetSynchEventFromPool(false);
	GTickWithoutSleepCount = 0;

	FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateStatic(&FEmbeddedCommunication::TickGameThread));
#endif
}

void FEmbeddedCommunication::ForceTick(int ID, float MinTimeSlice, float MaxTimeSlice)
{
#if BUILD_EMBEDDED_APP
	if (!HasMessagesInQueue())
	{
		UE_LOGF(LogInit, Display, "###ForceTick %d: no messages", ID);
		return;
	}

#if !UE_BUILD_SHIPPING
	// deal with overriding
	static bool bParsedOverrides = false;
	static float OverrideMinTimeSlice = -1.0f;
	static float OverrideMaxTimeSlice = -1.0f;
	if (!bParsedOverrides)
	{
		bParsedOverrides = true;
		
		FString Override;
		if (FParse::Value(FCommandLine::Get(), TEXT("ForceTickMin="), Override))
		{
			OverrideMinTimeSlice = FCString::Atoi(*Override);
		}
		if (FParse::Value(FCommandLine::Get(), TEXT("ForceTickMax="), Override))
		{
			OverrideMaxTimeSlice = FCString::Atoi(*Override);
		}
	}

	if (OverrideMinTimeSlice >= 0.0f)
	{
		MinTimeSlice = OverrideMinTimeSlice;
	}
	if (OverrideMaxTimeSlice >= 0.0f)
	{
		MaxTimeSlice = OverrideMaxTimeSlice;
	}
#endif

	double LastTime = FPlatformTime::Seconds();
	double EndMin = LastTime + MinTimeSlice;
	double EndMax = LastTime + MaxTimeSlice;
	double DeltaTime = 0.01;
	
	double Now = FPlatformTime::Seconds();
	while (Now < EndMin || (HasMessagesInQueue() && Now < EndMax))
	{
		UE_LOGF(LogInit, Display, "###ForceTick %d: processing messages...", ID);
		//We have to manually tick everything as we are looping the main thread here
		FTSTicker::GetCoreTicker().Tick(Now - LastTime);
		FThreadManager::Get().Tick();
		
		FPlatformProcess::Sleep(DeltaTime);

		// update timer
		LastTime = Now;
		Now = FPlatformTime::Seconds();
	}
	if (FPlatformTime::Seconds() > EndMax)
	{
		UE_LOGF(LogInit, Display, "  ###ForceTick %d timed out", ID);
	}
#endif
}

void FEmbeddedCommunication::KeepAwake(FName Requester, bool bNeedsRendering)
{
#if BUILD_EMBEDDED_APP
	FScopeLock Lock(&GEmbeddedLock);

	// use the relevant map
	TMap<FName, int>& WakeMap = bNeedsRendering ? GRenderingWakeMap : GTickWakeMap;

	// look for existing count
	int* WakeCount = WakeMap.Find(Requester);
	if (WakeCount == nullptr)
	{
		// make sure it's not in the other map since we only support it in only one
		checkf((bNeedsRendering ? GRenderingWakeMap : GTickWakeMap).Find(Requester) == nullptr, TEXT("Called KeepAwake with existing Requester (%s) that was previously had different bNeedsRendering (%d)"), *Requester.ToString(), bNeedsRendering);
		
		// start at 1
		WakeMap.Add(Requester, 1);
	}
	else
	{
		(*WakeCount)++;
	}
	
	WakeGameThread();
	
#endif
}

void FEmbeddedCommunication::AllowSleep(FName Requester)
{
#if BUILD_EMBEDDED_APP
	FScopeLock Lock(&GEmbeddedLock);
	
	// look in both maps
	bool bForRendering = true;
	int* WakeCount = GRenderingWakeMap.Find(Requester);
	if (WakeCount == nullptr)
	{
		WakeCount = GTickWakeMap.Find(Requester);
		bForRendering = false;
	}
	
	if (WakeCount == nullptr)
	{
		checkf(Requester == TEXT("Debug"), TEXT("Called a unmatched non-Debug AllowSleep, Requester = %s"), *Requester.ToString());
		return;
	}
	else
	{
		(*WakeCount)--;
		
		// at zero, remove the item
		if (*WakeCount == 0)
		{
			(bForRendering ? GRenderingWakeMap : GTickWakeMap).Remove(Requester);
		}
	}

#endif
}

#if BUILD_EMBEDDED_APP

DEFINE_LOG_CATEGORY_STATIC(LogBridge, Log, All);

#endif // BUILD_EMBEDDED_APP

void FEmbeddedCommunication::UELogFatal(const TCHAR* String)
{
#if BUILD_EMBEDDED_APP
	if (GWarn && UE_LOG_ACTIVE(LogBridge, Fatal))
	{
		GWarn->Log("LogBridge", ELogVerbosity::Fatal, String);
	}
#endif
}

void FEmbeddedCommunication::UELogError(const TCHAR* String)
{
#if BUILD_EMBEDDED_APP
	if (GWarn && UE_LOG_ACTIVE(LogBridge, Error))
	{
		GWarn->Log("LogBridge", ELogVerbosity::Error, String);
	}
#endif
}

void FEmbeddedCommunication::UELogWarning(const TCHAR* String)
{
#if BUILD_EMBEDDED_APP
	if (GWarn && UE_LOG_ACTIVE(LogBridge, Warning))
	{
		GWarn->Log("LogBridge", ELogVerbosity::Warning, String);
	}
#endif
}

void FEmbeddedCommunication::UELogDisplay(const TCHAR* String)
{
#if BUILD_EMBEDDED_APP
	if (GLog && UE_LOG_ACTIVE(LogBridge, Display))
	{
		GLog->Log("LogBridge", ELogVerbosity::Display, String);
	}
#endif
}

void FEmbeddedCommunication::UELogLog(const TCHAR* String)
{
#if BUILD_EMBEDDED_APP
	if (GLog && UE_LOG_ACTIVE(LogBridge, Log))
	{
		GLog->Log("LogBridge", ELogVerbosity::Log, String);
	}
#endif
}

void FEmbeddedCommunication::UELogVerbose(const TCHAR* String)
{
#if BUILD_EMBEDDED_APP
	if (GLog && UE_LOG_ACTIVE(LogBridge, Verbose))
	{
		GLog->Log("LogBridge", ELogVerbosity::Verbose, String);
	}
#endif
}

bool FEmbeddedCommunication::IsAwakeForTicking()
{
#if BUILD_EMBEDDED_APP
	FScopeLock Lock(&GEmbeddedLock);
	// if either map is awake, then tick
	return GRenderingWakeMap.Num() > 0 || GTickWakeMap.Num() > 0;
#else
	return true;
#endif
}

bool FEmbeddedCommunication::IsAwakeForRendering()
{
#if BUILD_EMBEDDED_APP
	FScopeLock Lock(&GEmbeddedLock);
	return GRenderingWakeMap.Num() > 0;
#else
	return true;
#endif
}


void FEmbeddedCommunication::RunOnGameThread(int Priority, TFunction<void()> Lambda)
{
#if BUILD_EMBEDDED_APP
	check(Priority < UE_ARRAY_COUNT(GEmbeddedQueues));

	{
		FScopeLock Lock(&GEmbeddedLock);
		GEmbeddedQueues[Priority].Enqueue(Lambda);

 		// wake up the game thread!
 		if (GSleepEvent)
 		{
 			GSleepEvent->Trigger();
 		}
	}
#endif
}

void FEmbeddedCommunication::WakeGameThread()
{
#if BUILD_EMBEDDED_APP
	// Allow 2 ticks without a sleep
	// Our sleep happens in the core ticker's tick, and that order gets reversed every tick,
	// so the caller isn't guaranteed to get a tick before our next sleep
	GTickWithoutSleepCount = 2;
	// wake up the game thread!
	if (GSleepEvent)
	{
		GSleepEvent->Trigger();
	}
#endif
}

bool FEmbeddedCommunication::TickGameThread(float DeltaTime)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FEmbeddedCommunication_TickGameThread);

#if BUILD_EMBEDDED_APP
	bool bEnableTickMultipleFunctors = false;
	GConfig->GetBool(TEXT("EmbeddedCommunication"), TEXT("bEnableTickMultipleFunctors"), bEnableTickMultipleFunctors, GEngineIni);
	double TickMaxTimeSeconds = 0.1;
	if (bEnableTickMultipleFunctors)
	{
		GConfig->GetDouble(TEXT("EmbeddedCommunication"), TEXT("TickMaxTimeSeconds"), TickMaxTimeSeconds, GEngineIni);
	}

	double TimeSliceEnd = FPlatformTime::Seconds() + TickMaxTimeSeconds;
	bool bLambdaWasCalled = false;
	do
	{
		TFunction<void()> LambdaToCall;
		{
			FScopeLock Lock(&GEmbeddedLock);

			for (int Queue = 0; Queue < UE_ARRAY_COUNT(GEmbeddedQueues); Queue++)
			{
				if (GEmbeddedQueues[Queue].Dequeue(LambdaToCall))
				{
					break;
				}
			}
		}

		if (LambdaToCall)
		{
			LambdaToCall();
			bLambdaWasCalled = true;
		}
		else
		{
			break;
		}
	}
	while (bEnableTickMultipleFunctors
		&& FPlatformTime::Seconds() < TimeSliceEnd);

	// sleep if nothing is going on
	if (!bLambdaWasCalled
		&& GRenderingWakeMap.Num() == 0
		&& GTickWakeMap.Num() == 0
		&& GTickWithoutSleepCount <= 0)
 	{
		double IdleSleepTimeSeconds = 5.0;
		GConfig->GetDouble(TEXT("EmbeddedCommunication"), TEXT("IdleSleepTimeSeconds"), IdleSleepTimeSeconds, GEngineIni);

		if (FEmbeddedDelegates::SleepTickDelegate.IsBound())
		{
			// Sleep in small bursts until 5 seconds has elapsed, or we are triggered, ticking the SleepTickDelegate between each one.

			double IdleSleepTickIntervalSeconds = 1.0 / 60;
			GConfig->GetDouble(TEXT("EmbeddedCommunication"), TEXT("IdleSleepTickIntervalSeconds"), IdleSleepTickIntervalSeconds, GEngineIni);

			bool bWasTriggered = false;
			double SleepTickTimeSliceEnd = FPlatformTime::Seconds() + IdleSleepTimeSeconds;
			do 
			{
				const double PlatformTimeBefore = FPlatformTime::Seconds();

				FEmbeddedDelegates::SleepTickDelegate.Broadcast();

				const double PlatformTimeNow = FPlatformTime::Seconds();
				const double TimeSpentInSleepTickDelegate = PlatformTimeNow - PlatformTimeBefore;
				const double TimeUntilTimeSliceEnd = SleepTickTimeSliceEnd - PlatformTimeNow;
				const double TimeRemainingThisSleepTickInterval = IdleSleepTickIntervalSeconds - TimeSpentInSleepTickDelegate;
				// Can be negative if we spent longer than the interval time in Broadcast, or if we're already past the SleepTickTimeSliceEnd.
				const double SleepTimeSeconds = FMath::Min(TimeUntilTimeSliceEnd, TimeRemainingThisSleepTickInterval);

				if (SleepTimeSeconds > 0.0)
				{
					UE_LOGF(LogInit, VeryVerbose, "FEmbeddedCommunication Sleeping GameThread for %ls seconds...", *FString::SanitizeFloat(SleepTimeSeconds));
					const uint32 SleepTimeMilliseconds = 1000 * SleepTimeSeconds;
					bWasTriggered = GSleepEvent->Wait(SleepTimeMilliseconds);
					UE_LOGF(LogInit, VeryVerbose, "FEmbeddedCommunication Woke up. Reason=[%ls]", bWasTriggered ? TEXT("Triggered") : TEXT("TimedOut"));
				}
			} while (!bWasTriggered &&
				FPlatformTime::Seconds() < SleepTickTimeSliceEnd);
		}
		else
		{
			// Sleep for 5 seconds or until triggered
			UE_LOGF(LogInit, VeryVerbose, "FEmbeddedCommunication Sleeping GameThread for %ls seconds...", *FString::SanitizeFloat(IdleSleepTimeSeconds));
			const uint32 IdleSleepTimeMilliseconds = 1000 * IdleSleepTimeSeconds;
			bool bWasTriggered = GSleepEvent->Wait(IdleSleepTimeMilliseconds);
			UE_LOGF(LogInit, VeryVerbose, "FEmbeddedCommunication Woke up. Reason=[%ls]", bWasTriggered ? TEXT("Triggered") : TEXT("TimedOut"));
		}
 	}
	if (GTickWithoutSleepCount > 0)
	{
		--GTickWithoutSleepCount;
	}
#endif

	return true;
}


FString FEmbeddedCommunication::GetDebugInfo()
{
#if BUILD_EMBEDDED_APP
	FScopeLock Lock(&GEmbeddedLock);

	FString Str;
	for (auto It : GRenderingWakeMap)
	{
		Str += FString::Printf(TEXT("%s:%d "), *It.Key.ToString(), It.Value);
	}
	Str += ("|");
	for (auto It : GTickWakeMap)
	{
		Str += FString::Printf(TEXT("%s:%d "), *It.Key.ToString(), It.Value);
	}
	return Str;
//	return FString::Printf(TEXT("%dR/%dT"), GRenderingWakeMap.Num(), GTickWakeMap.Num());
#else
	return TEXT("---");
#endif
}
