// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/EncryptionKeyManager.h"
#include "Async/UniqueLock.h"
#include "Misc/CoreDelegates.h"

namespace UE
{

FEncryptionKeyManager::FEncryptionKeyManager()
{
	if (FCoreDelegates::GetPakEncryptionKeyDelegate().IsBound())
	{
		FAES::FAESKey Key;
		FCoreDelegates::GetPakEncryptionKeyDelegate().Execute(Key.Key);

		if (Key.IsValid())
		{
			AddKey(FGuid(), Key);
		}
	}

	FCoreDelegates::GetRegisterEncryptionKeyMulticastDelegate().AddRaw(this, &FEncryptionKeyManager::AddKey);
}

FEncryptionKeyManager::~FEncryptionKeyManager()
{
	FCoreDelegates::GetRegisterEncryptionKeyMulticastDelegate().RemoveAll(this);
}

bool FEncryptionKeyManager::ContainsKey(const FGuid& Id) const
{
	TUniqueLock Lock(Mutex);
	return Keys.Contains(Id);
}

void FEncryptionKeyManager::AddKey(const FGuid& Id, const FAES::FAESKey& Key)
{
	bool bAdded = false;
	{
		TUniqueLock Lock(Mutex);

		if (!Keys.Contains(Id))
		{
			Keys.Add(Id, Key);
			bAdded = true;
		}
	}

	if (bAdded && KeyAdded.IsBound())
	{
		KeyAdded.Broadcast(Id, Key);
	}
}

bool FEncryptionKeyManager::TryGetKey(const FGuid& Id, FAES::FAESKey& OutKey) const
{
	TUniqueLock Lock(Mutex);
	if (const FAES::FAESKey* Key = Keys.Find(Id))
	{
		OutKey = *Key;
		return true;
	}

	return false;
}

TMap<FGuid, FAES::FAESKey> FEncryptionKeyManager::GetAllKeys() const
{
	TUniqueLock Lock(Mutex);
	return Keys;
}

FEncryptionKeyManager& FEncryptionKeyManager::Get()
{
	static FEncryptionKeyManager Mgr;
	return Mgr;
}

} // namespace UE
