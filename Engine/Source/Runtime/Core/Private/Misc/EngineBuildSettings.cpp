// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/EngineBuildSettings.h"

#include "CoreTypes.h"
#include "Misc/Paths.h"


bool FEngineBuildSettings::IsInternalBuild()
{
	static bool bIsInternalBuild = FPaths::FileExists( FPaths::EngineDir() / TEXT("Restricted/NotForLicensees/Build/EpicInternal.txt") );
	return bIsInternalBuild;
}


bool FEngineBuildSettings::IsPerforceBuild()
{
	static bool bIsPerforceBuild = FPaths::FileExists(FPaths::EngineDir() / TEXT("Build/PerforceBuild.txt"));
	return bIsPerforceBuild;
}


bool FEngineBuildSettings::IsSourceDistribution()
{
	return IsSourceDistribution( FPaths::RootDir() );
}


bool FEngineBuildSettings::IsSourceDistribution(const FString& RootDir)
{
	static bool bIsSourceDistribution = FPaths::FileExists(RootDir / TEXT("Engine/Build/SourceDistribution.txt"));
	return bIsSourceDistribution;
}
