// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/EngineVersion.h"

#include "BuildSettings.h"
#include "Math/NumericLimits.h"
#include "Misc/CString.h"
#include "Misc/Guid.h"
#include "Misc/LazySingleton.h"
#include "Runtime/Launch/Resources/Version.h"
#include "Serialization/Archive.h"
#include "Serialization/CustomVersion.h"
#include "Serialization/StructuredArchiveAdapters.h"
#include "Serialization/StructuredArchiveNameHelpers.h"
#include "Serialization/StructuredArchiveSlots.h"
#include "UObject/ReleaseObjectVersion.h"
#include "HAL/IConsoleManager.h" // for FAutoConsoleVariableRef

int32 GSkipChangelistCompatibilityVersionCheck = 0;
static FAutoConsoleVariableRef CSkipChangelistCompatibilityVersionCheck(
	TEXT("s.SkipChangelistCompatibilityVersionCheck"),
	GSkipChangelistCompatibilityVersionCheck,
	TEXT("If true, engine version compatibility checks will not use changelist numbers to determine compatibility. NOTE: Changelist compatibility checks are automatically skipped in licensee builds."),
	ECVF_Default);

FEngineVersionBase::FEngineVersionBase(uint16 InMajor, uint16 InMinor, uint16 InPatch, uint32 InChangelist)
: Major(InMajor)
, Minor(InMinor)
, Patch(InPatch)
, Changelist(InChangelist)
{
}

uint32 FEngineVersionBase::GetChangelist() const
{
	// Mask to ignore licensee bit
	return Changelist & (uint32)0x7fffffffU;
}

bool FEngineVersionBase::IsLicenseeVersion() const
{
	// Check for licensee bit
	return (Changelist & (uint32)0x80000000U) != 0;
}

bool FEngineVersionBase::IsEmpty() const
{
	return Major == 0 && Minor == 0 && Patch == 0;
}

bool FEngineVersionBase::HasChangelist() const
{
	return GetChangelist() != 0;
}

EVersionComparison FEngineVersionBase::GetNewest(const FEngineVersionBase &First, const FEngineVersionBase &Second, EVersionComponent *OutComponent)
{
	EVersionComponent LocalComponent = EVersionComponent::Minor;
	EVersionComponent& Component = OutComponent ? *OutComponent : LocalComponent;

	// Compare major versions
	if (First.GetMajor() != Second.GetMajor())
	{
		Component = EVersionComponent::Major;
		return (First.GetMajor() > Second.GetMajor()) ? EVersionComparison::First : EVersionComparison::Second;
	}

	// Compare minor versions
	if (First.GetMinor() != Second.GetMinor())
	{
		Component = EVersionComponent::Minor;
		return (First.GetMinor() > Second.GetMinor()) ? EVersionComparison::First : EVersionComparison::Second;
	}

	// Compare patch versions
	if (First.GetPatch() != Second.GetPatch())
	{
		Component = EVersionComponent::Patch;
		return (First.GetPatch() > Second.GetPatch()) ? EVersionComparison::First : EVersionComparison::Second;
	}

	// Compare changelists (only if they're both from the same vendor, and they're both valid)
	if (First.IsLicenseeVersion() == Second.IsLicenseeVersion() && First.HasChangelist() && Second.HasChangelist() && First.GetChangelist() != Second.GetChangelist())
	{
		Component = EVersionComponent::Changelist;
		return (First.GetChangelist() > Second.GetChangelist()) ? EVersionComparison::First : EVersionComparison::Second;
	}

	// Otherwise they're the same
	return EVersionComparison::Neither;
}

uint32 FEngineVersionBase::EncodeLicenseeChangelist(uint32 Changelist)
{
	return Changelist | 0x80000000;
}


FEngineVersion::FEngineVersion(uint16 InMajor, uint16 InMinor, uint16 InPatch, uint32 InChangelist, const FString &InBranch)
{
	Set(InMajor, InMinor, InPatch, InChangelist, InBranch);
}

void FEngineVersion::Set(uint16 InMajor, uint16 InMinor, uint16 InPatch, uint32 InChangelist, const FString &InBranch)
{
	Major = InMajor;
	Minor = InMinor;
	Patch = InPatch;
	Changelist = InChangelist;
	Branch = InBranch;
}

void FEngineVersion::Empty()
{
	Set(0, 0, 0, 0, FString());
}

bool FEngineVersion::ExactMatch(const FEngineVersion& Other) const
{
	return Major == Other.Major && Minor == Other.Minor && Patch == Other.Patch && Changelist == Other.Changelist && Branch == Other.Branch;
}

bool FEngineVersion::IsCompatibleWith(const FEngineVersionBase &Other) const
{
	// If this or the other is not a promoted build, always assume compatibility. 
	if(!HasChangelist() || !Other.HasChangelist())
	{
		return true;
	}
	else
	{
		EVersionComponent ConflictingComponent = EVersionComponent::Minor;
		EVersionComparison CompareResult = FEngineVersion::GetNewest(*this, Other, &ConflictingComponent);
		
		const bool bIsCompatible = (CompareResult != EVersionComparison::Second) || (GSkipChangelistCompatibilityVersionCheck && ConflictingComponent == EVersionComponent::Changelist);
		return bIsCompatible;
	}
}

FString FEngineVersion::ToString(EVersionComponent LastComponent) const
{
	FString Result = FString::Printf(TEXT("%u"), Major);
	if(LastComponent >= EVersionComponent::Minor)
	{
		Result += FString::Printf(TEXT(".%u"), Minor);
		if(LastComponent >= EVersionComponent::Patch)
		{
			Result += FString::Printf(TEXT(".%u"), Patch);
			if(LastComponent >= EVersionComponent::Changelist)
			{
				Result += FString::Printf(TEXT("-%u"), GetChangelist());
				if(LastComponent >= EVersionComponent::Branch && Branch.Len() > 0)
				{
					Result += FString::Printf(TEXT("+%s"), *Branch);
				}
			}
		}
	}
	return Result;
}

bool FEngineVersion::Parse(const FString &Text, FEngineVersion &OutVersion)
{
	TCHAR *End;

	// Read the major/minor/patch numbers
	uint64 Major = FCString::Strtoui64(*Text, &End, 10);
	if(Major > MAX_uint16 || *(End++) != '.') return false;

	uint64 Minor = FCString::Strtoui64(End, &End, 10);
	if(Minor > MAX_uint16 || *(End++) != '.') return false;

	uint64 Patch = FCString::Strtoui64(End, &End, 10);
	if(Patch > MAX_uint16) return false;

	// Read the optional changelist number
	uint64 Changelist = 0;
	if(*End == '-')
	{
		End++;
		Changelist = FCString::Strtoui64(End, &End, 10);
		if(Changelist > MAX_uint32) return false;
	}

	// Read the optional branch name
	FString Branch;
	if(*End == '+')
	{
		End++;
		// read to the end of the string. There's no standard for the branch name to verify.
		Branch = FString(End);
	}

	// Build the output version
	OutVersion.Set((uint16)Major, (uint16)Minor, (uint16)Patch, (uint32)Changelist, Branch);
	return true;
}

struct FGlobalEngineVersions
{
	const FEngineVersion Current;
	const FEngineVersion CompatibleWith;

	FGlobalEngineVersions()
		: Current(ENGINE_MAJOR_VERSION, ENGINE_MINOR_VERSION, ENGINE_PATCH_VERSION,
			BuildSettings::GetCurrentChangelist() | (BuildSettings::IsLicenseeVersion() ? (1U << 31) : 0), BuildSettings::GetBranchName())
		, CompatibleWith(ENGINE_MAJOR_VERSION, ENGINE_MINOR_VERSION, (BuildSettings::IsLicenseeVersion() ? ENGINE_PATCH_VERSION : 0),
			BuildSettings::GetCompatibleChangelist() | (BuildSettings::IsLicenseeVersion() ? (1U << 31) : 0), BuildSettings::GetBranchName())
	{}
};

const FEngineVersion& FEngineVersion::Current()
{
	return TLazySingleton<FGlobalEngineVersions>::Get().Current;
}

const FEngineVersion& FEngineVersion::CompatibleWith()
{
	return TLazySingleton<FGlobalEngineVersions>::Get().CompatibleWith;
}

void FEngineVersion::TearDown()
{
	TLazySingleton<FGlobalEngineVersions>::TearDown();
}

const FString& FEngineVersion::GetBranchDescriptor() const
{
	return Branch;
}

void operator<<(FArchive &Ar, FEngineVersion &Version)
{
	FStructuredArchiveFromArchive(Ar).GetSlot() << Version;
}

void operator<<(FStructuredArchive::FSlot Slot, FEngineVersion &Version)
{
	FArchive& BaseArchive = Slot.GetUnderlyingArchive();
	if (BaseArchive.IsTextFormat())
	{
		if (BaseArchive.IsLoading())
		{
			FString VersionString;
			Slot << VersionString;
			FEngineVersion::Parse(VersionString, Version);
		}
		else
		{
			FString VersionString = Version.ToString();
			Slot << VersionString;
		}
	}
	else
	{
		FStructuredArchive::FRecord Record = Slot.EnterRecord();
		Record << SA_VALUE(TEXT("Major"), Version.Major);
		Record << SA_VALUE(TEXT("Minor"), Version.Minor);
		Record << SA_VALUE(TEXT("Patch"), Version.Patch);
		Record << SA_VALUE(TEXT("Changelist"), Version.Changelist);
		Record << SA_VALUE(TEXT("Branch"), Version.Branch);
	}
}

// Unique Release Object version id
const FGuid FReleaseObjectVersion::GUID(0x9C54D522, 0xA8264FBE, 0x94210746, 0x61B482D0);
// Register Release custom version with Core
FCustomVersionRegistration GRegisterReleaseObjectVersion(FReleaseObjectVersion::GUID, FReleaseObjectVersion::LatestVersion, TEXT("Release"));
