// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "Containers/LockFreeList.h"
#include "HAL/PlatformProcess.h"
#include "HAL/Event.h"

#ifndef USE_EVENT_POOLING
	#define USE_EVENT_POOLING 1
#endif

class FSafeRecyclableEvent  final: public FEvent
{
public:
	FEvent* InnerEvent;

	FSafeRecyclableEvent(FEvent* InInnerEvent)
		: InnerEvent(InInnerEvent)
	{
	}

	~FSafeRecyclableEvent()
	{
		InnerEvent = nullptr;
	}

	virtual bool Create(bool bIsManualReset = false)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return InnerEvent->Create(bIsManualReset);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	virtual bool IsManualReset()
	{
		return InnerEvent->IsManualReset();
	}

	virtual void Trigger()
	{
		InnerEvent->Trigger();
	}

	virtual void Reset()
	{
		InnerEvent->Reset();
	}

	virtual bool Wait(uint32 WaitTime, const bool bIgnoreThreadIdleStats = false)
	{
		return InnerEvent->Wait(WaitTime, bIgnoreThreadIdleStats);
	}

};

/**
 * Template class for event pools.
 *
 * Events are expensive to create on most platforms. This pool allows for efficient
 * recycling of event instances that are no longer used. Events can have their signaled
 * state reset automatically or manually. The PoolType template parameter specifies
 * which type of events the pool managers.
 *
 * @param PoolType Specifies the type of pool.
 * @see FEvent
 */
template<EEventMode PoolType>
class TEventPool
{
public:
#if USE_EVENT_POOLING
	~TEventPool()
	{
		EmptyPool();
	}
#endif

	/**
	 * Gets an event from the pool or creates one if necessary.
	 *
	 * @return The event.
	 * @see ReturnToPool
	 */
	FEvent* GetEventFromPool()
	{
		return new FSafeRecyclableEvent(GetRawEvent());
	}

	/**
	 * Returns an event to the pool.
	 *
	 * @param Event The event to return.
	 * @see GetEventFromPool
	 */
	void ReturnToPool(FEvent* Event)
	{
		check(Event);
		check(Event->IsManualReset() == (PoolType == EEventMode::ManualReset));

		FSafeRecyclableEvent* SafeEvent = (FSafeRecyclableEvent*)Event;
		FEvent* InnerEvent = SafeEvent->InnerEvent;
		// Make sure the safeevent can't be used anymore before returning the inner event to the pool
		// This will help trap use-after-free of events that can end up triggering events while they're in the pool
		// and causing issues for other pool users.
		delete SafeEvent;
		ReturnRawEvent(InnerEvent);
	}

	void EmptyPool()
	{
#if USE_EVENT_POOLING
		while (FEvent* Event = Pool.Pop())
		{
			delete Event;
		}
#endif
	}

	/**
	* Gets a "raw" event (as opposite to `FSafeRecyclableEvent` handle returned by `GetEventFromPool`) from the pool or 
	* creates one if necessary.
	* @see ReturnRaw
	*/
	FEvent* GetRawEvent()
	{
		FEvent* Event =
#if USE_EVENT_POOLING
			Pool.Pop();
#else
			nullptr;
#endif

		if (Event == nullptr)
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			Event = FPlatformProcess::CreateSynchEvent(PoolType == EEventMode::ManualReset);
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}

		check(Event);

		Event->AdvanceStats();

		return Event;
	}

	/**
	 * Returns a "raw" event to the pool.
	 * @see GetRaw
	 */
	void ReturnRawEvent(FEvent* Event)
	{
		check(Event);

#if USE_EVENT_POOLING
		Event->Reset();
		Pool.Push(Event);
#else
		delete Event;
#endif	
	}

private:
#if USE_EVENT_POOLING
	/** Holds the collection of recycled events. */
	TLockFreePointerListUnordered<FEvent, PLATFORM_CACHE_LINE_SIZE> Pool;
#endif
};
