// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/Exec.h"

#include "Misc/AssertionMacros.h"
#include "Misc/CoreDelegates.h"

FExec::~FExec()
{
}

#if UE_ALLOW_EXEC_COMMANDS
bool FExec::Exec( UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar )
{
	bool bExecSuccess = false;

#if UE_ALLOW_EXEC_EDITOR
	bExecSuccess = bExecSuccess || Exec_Editor( InWorld, Cmd, Ar );
#endif

#if UE_ALLOW_EXEC_DEV
	bExecSuccess = bExecSuccess || Exec_Dev( InWorld, Cmd, Ar );
#endif

	bExecSuccess = bExecSuccess || Exec_Runtime( InWorld, Cmd, Ar );

	return bExecSuccess;
}
#else // UE_ALLOW_EXEC_COMMANDS
bool FExec::Exec( UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar )
{
	return Exec( InWorld, Cmd );
}

bool FExec::Exec( UWorld* InWorld, const TCHAR* Cmd )
{
	ensure(false);
	FCoreDelegates::OnDisallowedExecCommandCalled.Broadcast( Cmd );

	return false;
}
#endif // !UE_ALLOW_EXEC_COMMANDS