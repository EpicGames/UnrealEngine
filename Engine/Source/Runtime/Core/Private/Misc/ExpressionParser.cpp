// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/ExpressionParser.h"

#include "Containers/AnsiString.h"
#include "Containers/Utf8String.h"
#include "Misc/AutomationTest.h"
#include "Misc/TextFilterExpressionEvaluator.h"
#include "Math/BasicMathExpressionEvaluator.h"

#define LOCTEXT_NAMESPACE "ExpressionParser"


template <typename CharType>
TTokenStream<CharType>::TTokenStream(const CharType* In)
	: Start(In)
	, End(Start + TCString<CharType>::Strlen(Start))
	, ReadPos(In)
{
}

template <typename CharType>
bool TTokenStream<CharType>::IsReadPosValid(const CharType* InPos, int32 MinNumChars) const
{
	return InPos >= Start && InPos <= End - MinNumChars;
}

template <typename CharType>
CharType TTokenStream<CharType>::PeekChar(int32 Offset) const
{
	if (ReadPos + Offset < End)
	{
		return *(ReadPos + Offset);
	}

	return CHARTEXT(CharType, '\0');
}

template <typename CharType>
int32 TTokenStream<CharType>::CharsRemaining() const
{
	return UE_PTRDIFF_TO_INT32(End - ReadPos);
}

template <typename CharType>
bool TTokenStream<CharType>::IsEmpty() const
{
	return ReadPos == End;
}

template <typename CharType>
int32 TTokenStream<CharType>::GetPosition() const
{
	return UE_PTRDIFF_TO_INT32(ReadPos - Start);
}

template <typename CharType>
FString TTokenStream<CharType>::GetErrorContext() const
{
	const CharType* StartPos = ReadPos;
	const CharType* EndPos = StartPos;

	// Skip over any leading whitespace
	while (TChar<CharType>::IsWhitespace(*EndPos))
	{
		EndPos++;
	}

	// Read until next whitespace or end of string
	while (!TChar<CharType>::IsWhitespace(*EndPos) && *EndPos != '\0')
	{
		EndPos++;
	}

	static const int32 MaxChars = 32;
	FString Context = FString::ConstructFromPtrSize(StartPos, FMath::Min(int32(EndPos - StartPos), MaxChars));
	if (EndPos - StartPos > MaxChars)
	{
		Context += TEXT("...");
	}
	return Context;
}

/** Parse out a token */
template <typename CharType>
TOptional<TStringToken<CharType>> TTokenStream<CharType>::ParseToken(TFunctionRef<EParseState(CharType)> Pred, TStringToken<CharType>* Accumulate) const
{
	const CharType* OptReadPos = Accumulate ? Accumulate->GetTokenEndPos() : ReadPos;

	if (!IsReadPosValid(OptReadPos))
	{
		return {};
	}

	TStringToken<CharType> Token(OptReadPos, 0, UE_PTRDIFF_TO_INT32(OptReadPos - Start));

	while (Token.GetTokenEndPos() != End)
	{
		const EParseState State = Pred(*Token.GetTokenEndPos());
		
		if (State == EParseState::Cancel)
		{
			return {};
		}
		
		if (State == EParseState::Continue || State == EParseState::StopAfter)
		{
			// Need to include this character in this token
			++Token.TokenEnd;
		}

		if (State == EParseState::StopAfter || State == EParseState::StopBefore)
		{
			// Finished parsing the token
			break;
		}
	}

	if (Token.IsValid())
	{
		if (Accumulate)
		{
			Accumulate->Accumulate(Token);
		}
		return Token;
	}

	return {};
}

template <typename CharType>
TOptional<TStringToken<CharType>> TTokenStream<CharType>::ParseSymbol(TStringToken<CharType>* Accumulate) const
{
	const CharType* OptReadPos = Accumulate ? Accumulate->GetTokenEndPos() : ReadPos;

	if (!IsReadPosValid(OptReadPos))
	{
		return {};
	}
	
	TStringToken<CharType> Token(OptReadPos, 0, UE_PTRDIFF_TO_INT32(OptReadPos - Start));
	++Token.TokenEnd;

	if (Accumulate)
	{
		Accumulate->Accumulate(Token);
	}

	return Token;
}

template <typename CharType>
TOptional<TStringToken<CharType>> TTokenStream<CharType>::ParseSymbol(CharType Symbol, TStringToken<CharType>* Accumulate) const
{
	const CharType* OptReadPos = Accumulate ? Accumulate->GetTokenEndPos() : ReadPos;

	if (!IsReadPosValid(OptReadPos))
	{
		return {};
	}
	
	TStringToken<CharType> Token(OptReadPos, 0, UE_PTRDIFF_TO_INT32(OptReadPos - Start));

	if (*Token.TokenEnd == Symbol)
	{
		++Token.TokenEnd;

		if (Accumulate)
		{
			Accumulate->Accumulate(Token);
		}

		return Token;
	}

	return {};
}

template <typename CharType>
TOptional<TStringToken<CharType>> TTokenStream<CharType>::ParseToken(const CharType* Symbol, TStringToken<CharType>* Accumulate) const
{
	const CharType* OptReadPos = Accumulate ? Accumulate->GetTokenEndPos() : ReadPos;

	const int32 Len = TCString<CharType>::Strlen(Symbol);
	if (!IsReadPosValid(OptReadPos, Len))
	{
		return {};
	}

	if (*OptReadPos != *Symbol)
	{
		return {};
	}

	TStringToken<CharType> Token(OptReadPos, 0, UE_PTRDIFF_TO_INT32(OptReadPos - Start));
	
	if (TCString<CharType>::Strncmp(Token.GetTokenEndPos(), Symbol, Len) == 0)
	{
		Token.TokenEnd += Len;

		if (Accumulate)
		{
			Accumulate->Accumulate(Token);
		}

		return Token;
	}

	return {};
}

template <typename CharType>
TOptional<TStringToken<CharType>> TTokenStream<CharType>::ParseTokenIgnoreCase(const CharType* Symbol, TStringToken<CharType>* Accumulate) const
{
	const CharType* OptReadPos = Accumulate ? Accumulate->GetTokenEndPos() : ReadPos;

	const int32 Len = TCString<CharType>::Strlen(Symbol);
	if (!IsReadPosValid(OptReadPos, Len))
	{
		return {};
	}

	TStringToken<CharType> Token(OptReadPos, 0, UE_PTRDIFF_TO_INT32(OptReadPos - Start));

	if (TCString<CharType>::Strnicmp(OptReadPos, Symbol, Len) == 0)
	{
		Token.TokenEnd += Len;

		if (Accumulate)
		{
			Accumulate->Accumulate(Token);
		}

		return Token;
	}

	return {};
}

template <typename CharType>
TOptional<TStringToken<CharType>> TTokenStream<CharType>::ParseWhitespace(TStringToken<CharType>* Accumulate) const
{
	const CharType* OptReadPos = Accumulate ? Accumulate->GetTokenEndPos() : ReadPos;

	if (IsReadPosValid(OptReadPos))
	{
		return ParseToken([](CharType InC){ return TChar<CharType>::IsWhitespace(InC) ? EParseState::Continue : EParseState::StopBefore; }, Accumulate);
	}

	return {};
}

template <typename CharType>
TOptional<TStringToken<CharType>> TTokenStream<CharType>::GenerateToken(int32 NumChars, TStringToken<CharType>* Accumulate) const
{
	const CharType* OptReadPos = Accumulate ? Accumulate->GetTokenEndPos() : ReadPos;

	if (IsReadPosValid(OptReadPos, NumChars))
	{
		TStringToken<CharType> Token(OptReadPos, 0, UE_PTRDIFF_TO_INT32(OptReadPos - Start));
		Token.TokenEnd += NumChars;
		if (Accumulate)
		{
			Accumulate->Accumulate(Token);
		}
		return Token;
	}

	return {};
}

template <typename CharType>
void TTokenStream<CharType>::SetReadPos(const TStringToken<CharType>& Token)
{
	if (ensure(IsReadPosValid(Token.TokenEnd, 0)))
	{
		ReadPos = Token.TokenEnd;
	}
}

template <typename CharType>
TExpressionTokenConsumer<CharType>::TExpressionTokenConsumer(const CharType* InExpression)
	: Stream(InExpression)
{
}

template <typename CharType>
TArray<TExpressionToken<CharType>> TExpressionTokenConsumer<CharType>::Extract()
{
	TArray<TExpressionToken<CharType>> Swapped;
	Swap(Swapped, Tokens);
	return Swapped;
}

template <typename CharType>
void TExpressionTokenConsumer<CharType>::Add(const TStringToken<CharType>& SourceToken, FExpressionNode&& Node)
{
	Stream.SetReadPos(SourceToken);
	Tokens.Emplace(SourceToken, MoveTemp(Node));
}

template <typename CharType>
TTokenStream<CharType>& TExpressionTokenConsumer<CharType>::GetStream()
{
	return Stream;
}

template <typename CharType>
void TTokenDefinitions<CharType>::IgnoreWhitespace()
{
	bIgnoreWhitespace = true;
}

template <typename CharType>
void TTokenDefinitions<CharType>::DefineToken(TFunction<TExpressionDefinition<CharType>>&& Definition)
{
	Definitions.Emplace(MoveTemp(Definition));
}

template <typename CharType>
bool TTokenDefinitions<CharType>::DoesIgnoreWhitespace()
{
	return bIgnoreWhitespace;
}

template <typename CharType>
TOptional<FExpressionError> TTokenDefinitions<CharType>::ConsumeToken(TExpressionTokenConsumer<CharType>& Consumer) const
{
	auto& Stream = Consumer.GetStream();
	
	// Skip over whitespace
	if (bIgnoreWhitespace)
	{
		TOptional<TStringToken<CharType>> Whitespace = Stream.ParseWhitespace();
		if (Whitespace.IsSet())
		{
			Stream.SetReadPos(Whitespace.GetValue());
		}
	}

	if (Stream.IsEmpty())
	{
		// Trailing whitespace in the expression.
		return {};
	}

	const auto* Pos = Stream.GetRead();

	// Try each token in turn. First come first served.
	for (const auto& Def : Definitions)
	{
		// Call the token definition
		auto Error = Def(Consumer);
		if (Error.IsSet())
		{
			return Error;
		}
		// If the stream has moved on, the definition added one or more tokens, so 
		else if (Stream.GetRead() != Pos)
		{
			return {};
		}
	}

	// No token definition matched the stream at its current position - fatal error
	FFormatOrderedArguments Args;
	Args.Add(FText::FromString(Consumer.GetStream().GetErrorContext()));
	Args.Add(Consumer.GetStream().GetPosition());
	return FExpressionError(FText::Format(LOCTEXT("LexicalError", "Unrecognized token '{0}' at character {1}"), Args));
}

template <typename CharType>
TOptional<FExpressionError> TTokenDefinitions<CharType>::ConsumeTokens(TExpressionTokenConsumer<CharType>& Consumer) const
{
	auto& Stream = Consumer.GetStream();
	while(!Stream.IsEmpty())
	{
		auto Error = ConsumeToken(Consumer);
		if (Error.IsSet())
		{
			return Error;
		}
	}

	return {};
}

FExpressionNode::~FExpressionNode()
{
	if (auto* Data = GetData())
	{
		Data->~IExpressionNodeStorage();
	}
}

FExpressionNode::FExpressionNode(FExpressionNode&& In)
{
	*this = MoveTemp(In);
}

FExpressionNode& FExpressionNode::operator=(FExpressionNode&& In)
{
	if (TypeId == In.TypeId && TypeId.IsValid())
	{
		// If we have the same types, we can move-assign properly
		In.GetData()->MoveAssign(InlineBytes);
	}
	else
	{
		// Otherwise we have to destroy what we have, and reseat the RHS
		if (auto* ThisData = GetData())
		{
			ThisData->~IExpressionNodeStorage();
		}

		TypeId = In.TypeId;
		if (auto* SrcData = In.GetData())
		{
			SrcData->Reseat(InlineBytes);

			// Empty the RHS
			In.TypeId = FGuid();
			SrcData->~IExpressionNodeStorage();
		}
	}

	return *this;
}

const FGuid& FExpressionNode::GetTypeId() const
{
	return TypeId;
}

Impl::IExpressionNodeStorage* FExpressionNode::GetData()
{
	return TypeId.IsValid() ? reinterpret_cast<Impl::IExpressionNodeStorage*>(InlineBytes) : nullptr;
}

const Impl::IExpressionNodeStorage* FExpressionNode::GetData() const
{
	return TypeId.IsValid() ? reinterpret_cast<const Impl::IExpressionNodeStorage*>(InlineBytes) : nullptr;
}

FExpressionNode FExpressionNode::Copy() const
{
	if (const auto* Data = GetData())
	{
		return Data->Copy();
	}
	return FExpressionNode();
}

const FGuid* FExpressionGrammar::GetGrouping(const FGuid& TypeId) const
{
	return Groupings.Find(TypeId);
}

bool FExpressionGrammar::HasPreUnaryOperator(const FGuid& InTypeId) const
{
	return PreUnaryOperators.Contains(InTypeId);
}

bool FExpressionGrammar::HasPostUnaryOperator(const FGuid& InTypeId) const
{
	return PostUnaryOperators.Contains(InTypeId);
}

const FOpParameters* FExpressionGrammar::GetBinaryOperatorDefParameters(const FGuid& InTypeId) const
{
	return BinaryOperators.Find(InTypeId);
}

template <typename CharType>
struct TExpressionCompiler
{
	TExpressionCompiler(const FExpressionGrammar& InGrammar, TArray<TExpressionToken<CharType>>& InTokens)
		: Grammar(InGrammar)
		, Tokens(InTokens)
	{
		CurrentTokenIndex = 0;
		Commands.Reserve(Tokens.Num());
	}

	TValueOrError<TArray<TCompiledToken<CharType>>, FExpressionError> Compile()
	{
		auto Error = CompileGroup(nullptr, nullptr);
		if (Error.IsSet())
		{
			return MakeError(Error.GetValue());
		}
		return MakeValue(MoveTemp(Commands));
	}

	struct FWrappedOperator : FNoncopyable
	{
		explicit FWrappedOperator(TCompiledToken<CharType> InToken, int32 InPrecedence = 0, int32 InShortCircuitIndex = INDEX_NONE)
			: Token(MoveTemp(InToken))
			, Precedence(InPrecedence)
			, ShortCircuitIndex(InShortCircuitIndex)
		{
		}

		FWrappedOperator(FWrappedOperator&& In)
			: Token(MoveTemp(In.Token))
			, Precedence(In.Precedence)
		{
		}

		FWrappedOperator& operator=(FWrappedOperator&& In)
		{
			Token = MoveTemp(In.Token);
			Precedence = In.Precedence;
			return *this;
		}

		TCompiledToken<CharType> Steal()
		{
			return MoveTemp(Token);
		}

		TCompiledToken<CharType> Token;
		int32 Precedence;
		int32 ShortCircuitIndex;
	};

	TOptional<FExpressionError> CompileGroup(const TExpressionToken<CharType>* GroupStart, const FGuid* StopAt)
	{
		enum class EState { PreUnary, PostUnary, Binary };

		TArray<FWrappedOperator> OperatorStack;
		OperatorStack.Reserve(Tokens.Num() - CurrentTokenIndex);

		auto PopOperator = [&]
		{
			int32 ShortCircuitIndex = OperatorStack.Last().ShortCircuitIndex;

			Commands.Add(OperatorStack.Pop(EAllowShrinking::No).Steal());
			if (ShortCircuitIndex != INDEX_NONE)
			{
				Commands[ShortCircuitIndex].ShortCircuitIndex = Commands.Num() - 1;
			}
		};

		bool bFoundEndOfGroup = StopAt == nullptr;

		// Start off looking for a unary operator
		EState State = EState::PreUnary;
		for (; CurrentTokenIndex < Tokens.Num(); ++CurrentTokenIndex)
		{
			auto& Token = Tokens[CurrentTokenIndex];
			const auto& TypeId = Token.Node.GetTypeId();

			if (const FGuid* GroupingEnd = Grammar.GetGrouping(TypeId))
			{
				// Ignore this token
				CurrentTokenIndex++;

				// Start of group - recurse
				auto Error = CompileGroup(&Token, GroupingEnd);

				if (Error.IsSet())
				{
					return Error;
				}

				State = EState::PostUnary;
			}
			else if (StopAt && TypeId == *StopAt)
			{
				// End of group
				bFoundEndOfGroup = true;
				break;
			}
			else if (State == EState::PreUnary)
			{
				if (Grammar.HasPreUnaryOperator(TypeId))
				{
					// Make this a unary op
					OperatorStack.Emplace(TCompiledToken<CharType>(TCompiledToken<CharType>::PreUnaryOperator, MoveTemp(Token)));
				}
				else if (Grammar.GetBinaryOperatorDefParameters(TypeId))
				{
					return FExpressionError(FText::Format(LOCTEXT("SyntaxError_NoBinaryOperand", "Syntax error: No operand specified for operator '{0}'"), FText::FromString(Token.Context.GetString())));
				}
				else if (Grammar.HasPostUnaryOperator(TypeId))
				{
					// Found a post-unary operator for the preceeding token
					State = EState::PostUnary;

					// Pop off any pending unary operators
					while (OperatorStack.Num() > 0 && OperatorStack.Last().Precedence <= 0)
					{
						PopOperator();
					}

					// Make this a post-unary op
					OperatorStack.Emplace(TCompiledToken<CharType>(TCompiledToken<CharType>::PostUnaryOperator, MoveTemp(Token)));
				}
				else
				{
					// Not an operator, so treat it as an ordinary token
					Commands.Add(TCompiledToken<CharType>(TCompiledToken<CharType>::Operand, MoveTemp(Token)));
					State = EState::PostUnary;
				}
			}
			else if (State == EState::PostUnary)
			{
				if (Grammar.HasPostUnaryOperator(TypeId))
				{
					// Pop off any pending unary operators
					while (OperatorStack.Num() > 0 && OperatorStack.Last().Precedence <= 0)
					{
						PopOperator();
					}

					// Make this a post-unary op
					OperatorStack.Emplace(TCompiledToken<CharType>(TCompiledToken<CharType>::PostUnaryOperator, MoveTemp(Token)));
				}
				else
				{
					// Checking for binary operators
					if (const FOpParameters* OpParms = Grammar.GetBinaryOperatorDefParameters(TypeId))
					{
						auto CheckPrecedence = [OpParms](int32 LastPrec, int32 Prec)
							{
								return (OpParms->Associativity == EAssociativity::LeftToRight ? (LastPrec <= Prec) : (LastPrec < Prec));
							};

						// Pop off anything of higher (or equal, if LTR associative) precedence than this one onto the command stack
						while (OperatorStack.Num() > 0 && CheckPrecedence(OperatorStack.Last().Precedence, OpParms->Precedence))
						{
							PopOperator();
						}

						int32 ShortCircuitIndex = INDEX_NONE;
						if (OpParms->bCanShortCircuit)
						{
							Commands.Add(TCompiledToken<CharType>(TCompiledToken<CharType>::ShortCircuit, TExpressionToken<CharType>(Token.Context, Token.Node.Copy()), Commands.Num()));
							ShortCircuitIndex = Commands.Num() - 1;
						}

						// Add the operator itself to the op stack
						OperatorStack.Emplace(TCompiledToken<CharType>(TCompiledToken<CharType>::BinaryOperator, MoveTemp(Token)), OpParms->Precedence, ShortCircuitIndex);

						// Check for a unary op again
						State = EState::PreUnary;
					}
					else
					{
						// Just add the token. It's possible that this is a syntax error (there's no binary operator specified between two tokens),
						// But we don't have enough information at this point to say whether or not it is an error
						Commands.Add(TCompiledToken<CharType>(TCompiledToken<CharType>::Operand, MoveTemp(Token)));
						State = EState::PreUnary;
					}
				}
			}
		}

		if (!bFoundEndOfGroup)
		{
			return FExpressionError(FText::Format(LOCTEXT("SyntaxError_UnmatchedGroup", "Syntax error: Reached end of expression before matching end of group '{0}' at line {1}:{2}"),
				FText::FromString(GroupStart->Context.GetString()),
				FText::AsNumber(GroupStart->Context.GetLineNumber()),
				FText::AsNumber(GroupStart->Context.GetCharacterIndex())
			));
		}

		// Pop everything off the operator stack, onto the command stack
		while (OperatorStack.Num() > 0)
		{
			PopOperator();
		}

		return {};
	}


private:

	int32 CurrentTokenIndex;

	/** Working structures */
	TArray<TCompiledToken<CharType>> Commands;

private:
	/** Const data provided by the parser */
	const FExpressionGrammar& Grammar;
	TArray<TExpressionToken<CharType>>& Tokens;
};

namespace ExpressionParser
{
	template <typename CharType>
	TLexResultType<CharType> Lex(const CharType* InExpression, const TTokenDefinitions<CharType>& TokenDefinitions)
	{
		TExpressionTokenConsumer<CharType> TokenConsumer(InExpression);
		
		TOptional<FExpressionError> Error = TokenDefinitions.ConsumeTokens(TokenConsumer);
		if (Error.IsSet())
		{
			return MakeError(Error.GetValue());
		}

		return MakeValue(TokenConsumer.Extract());
	}

	template <typename CharType>
	TCompileResultType<CharType> Compile(const CharType* InExpression, const TTokenDefinitions<CharType>& InTokenDefinitions, const FExpressionGrammar& InGrammar)
	{
		TValueOrError<TArray<TExpressionToken<CharType>>, FExpressionError> Result = Lex(InExpression, InTokenDefinitions);

		if (!Result.IsValid())
		{
			return MakeError(Result.GetError());
		}

		return Compile(MoveTemp(Result.GetValue()), InGrammar);
	}

	template <typename CharType>
	TCompileResultType<CharType> Compile(TArray<TExpressionToken<CharType>> InTokens, const FExpressionGrammar& InGrammar)
	{
		return TExpressionCompiler<CharType>(InGrammar, InTokens).Compile();
	}

	template <typename CharType>
	FExpressionResult Evaluate(const CharType* InExpression, const TTokenDefinitions<CharType>& InTokenDefinitions, const FExpressionGrammar& InGrammar, const TIOperatorEvaluationEnvironment<CharType>& InEnvironment)
	{
		TValueOrError<TArray<TCompiledToken<CharType>>, FExpressionError> CompilationResult = Compile(InExpression, InTokenDefinitions, InGrammar);

		if (!CompilationResult.IsValid())
		{
			return MakeError(CompilationResult.GetError());
		}

		return Evaluate(CompilationResult.GetValue(), InEnvironment);
	}

	template <typename CharType>
	FExpressionResult Evaluate(const TArray<TCompiledToken<CharType>>& CompiledTokens, const TIOperatorEvaluationEnvironment<CharType>& InEnvironment)
	{
		// Evaluation strategy: the supplied compiled tokens are const. To avoid copying the whole array, we store a separate array of
		// any tokens that are generated at runtime by the evaluator. The operand stack will consist of indices into either the CompiledTokens
		// array, or the RuntimeGeneratedTokens (where Index >= CompiledTokens.Num())
		TArray<TExpressionToken<CharType>> RuntimeGeneratedTokens;
		TArray<int32> OperandStack;

		/** Get the token pertaining to the specified operand index */
		auto GetToken = [&](int32 Index) -> const TExpressionToken<CharType>& {
			if (Index < CompiledTokens.Num())
			{
				return CompiledTokens[Index];
			}

			return RuntimeGeneratedTokens[Index - CompiledTokens.Num()];
		};

		/** Add a new token to the runtime generated array */
		auto AddToken = [&](TExpressionToken<CharType>&& In) -> int32 {
			auto Index = CompiledTokens.Num() + RuntimeGeneratedTokens.Num();
			RuntimeGeneratedTokens.Emplace(MoveTemp(In));
			return Index;
		};


		for (int32 Index = 0; Index < CompiledTokens.Num(); ++Index)
		{
			const auto& Token = CompiledTokens[Index];

			switch(Token.Type)
			{
			case TCompiledToken<CharType>::Benign:
				continue;

			case TCompiledToken<CharType>::Operand:
				OperandStack.Push(Index);
				continue;

			case TCompiledToken<CharType>::ShortCircuit:
				if (OperandStack.Num() >= 1 && Token.ShortCircuitIndex.IsSet() && InEnvironment.ShouldShortCircuit(Token, GetToken(OperandStack.Last())))
				{
					Index = Token.ShortCircuitIndex.GetValue();
				}
				continue;

			case TCompiledToken<CharType>::BinaryOperator:
				if (OperandStack.Num() >= 2)
				{
					// Binary
					const auto& R = GetToken(OperandStack.Pop());
					const auto& L = GetToken(OperandStack.Pop());

					auto OpResult = InEnvironment.ExecBinary(Token, L, R);
					if (OpResult.IsValid())
					{
						// Inherit the LHS context
						OperandStack.Push(AddToken(TExpressionToken<CharType>(L.Context, MoveTemp(OpResult.GetValue()))));
					}
					else
					{
						return MakeError(OpResult.GetError());
					}
				}
				else
				{
					FFormatOrderedArguments Args;
					Args.Add(FText::FromString(Token.Context.GetString()));
					return MakeError(FText::Format(LOCTEXT("SyntaxError_NotEnoughOperandsBinary", "Not enough operands for binary operator {0}"), Args));
				}
				break;
			
			case TCompiledToken<CharType>::PostUnaryOperator:
			case TCompiledToken<CharType>::PreUnaryOperator:

				if (OperandStack.Num() >= 1)
				{
					const auto& Operand = GetToken(OperandStack.Pop());

					FExpressionResult OpResult = (Token.Type == TCompiledToken<CharType>::PreUnaryOperator) ?
						InEnvironment.ExecPreUnary(Token, Operand) :
						InEnvironment.ExecPostUnary(Token, Operand);

					if (OpResult.IsValid())
					{
						// Inherit the LHS context
						OperandStack.Push(AddToken(TExpressionToken<CharType>(Operand.Context, MoveTemp(OpResult.GetValue()))));
					}
					else
					{
						return MakeError(OpResult.GetError());
					}
				}
				else
				{
					FFormatOrderedArguments Args;
					Args.Add(FText::FromString(Token.Context.GetString()));
					return MakeError(FText::Format(LOCTEXT("SyntaxError_NoUnaryOperand", "No operand for unary operator {0}"), Args));
				}
				break;
			}
		}

		if (OperandStack.Num() == 1)
		{
			return MakeValue(GetToken(OperandStack[0]).Node.Copy());
		}

		return MakeError(LOCTEXT("SyntaxError_InvalidExpression", "Could not evaluate expression"));
	}

	template CORE_API TLexResultType<ANSICHAR> Lex<ANSICHAR>(const ANSICHAR* InExpression, const TTokenDefinitions<ANSICHAR>& TokenDefinitions);
	template CORE_API TLexResultType<UTF8CHAR> Lex<UTF8CHAR>(const UTF8CHAR* InExpression, const TTokenDefinitions<UTF8CHAR>& TokenDefinitions);
	template CORE_API TLexResultType<WIDECHAR> Lex<WIDECHAR>(const WIDECHAR* InExpression, const TTokenDefinitions<WIDECHAR>& TokenDefinitions);

	template CORE_API TCompileResultType<ANSICHAR> Compile<ANSICHAR>(const ANSICHAR* InExpression, const TTokenDefinitions<ANSICHAR>& TokenDefinitions, const FExpressionGrammar& InGrammar);
	template CORE_API TCompileResultType<UTF8CHAR> Compile<UTF8CHAR>(const UTF8CHAR* InExpression, const TTokenDefinitions<UTF8CHAR>& TokenDefinitions, const FExpressionGrammar& InGrammar);
	template CORE_API TCompileResultType<WIDECHAR> Compile<WIDECHAR>(const WIDECHAR* InExpression, const TTokenDefinitions<WIDECHAR>& TokenDefinitions, const FExpressionGrammar& InGrammar);

	template CORE_API TCompileResultType<ANSICHAR> Compile<ANSICHAR>(TArray<TExpressionToken<ANSICHAR>> InTokens, const FExpressionGrammar& InGrammar);
	template CORE_API TCompileResultType<UTF8CHAR> Compile<UTF8CHAR>(TArray<TExpressionToken<UTF8CHAR>> InTokens, const FExpressionGrammar& InGrammar);
	template CORE_API TCompileResultType<WIDECHAR> Compile<WIDECHAR>(TArray<TExpressionToken<WIDECHAR>> InTokens, const FExpressionGrammar& InGrammar);

	template CORE_API FExpressionResult Evaluate<ANSICHAR>(const ANSICHAR* InExpression, const TTokenDefinitions<ANSICHAR>& InTokenDefinitions, const FExpressionGrammar& InGrammar, const TIOperatorEvaluationEnvironment<ANSICHAR>& InEnvironment);
	template CORE_API FExpressionResult Evaluate<UTF8CHAR>(const UTF8CHAR* InExpression, const TTokenDefinitions<UTF8CHAR>& InTokenDefinitions, const FExpressionGrammar& InGrammar, const TIOperatorEvaluationEnvironment<UTF8CHAR>& InEnvironment);
	template CORE_API FExpressionResult Evaluate<WIDECHAR>(const WIDECHAR* InExpression, const TTokenDefinitions<WIDECHAR>& InTokenDefinitions, const FExpressionGrammar& InGrammar, const TIOperatorEvaluationEnvironment<WIDECHAR>& InEnvironment);

	template CORE_API FExpressionResult Evaluate<ANSICHAR>(const TArray<TCompiledToken<ANSICHAR>>& CompiledTokens, const TIOperatorEvaluationEnvironment<ANSICHAR>& InEnvironment);
	template CORE_API FExpressionResult Evaluate<UTF8CHAR>(const TArray<TCompiledToken<UTF8CHAR>>& CompiledTokens, const TIOperatorEvaluationEnvironment<UTF8CHAR>& InEnvironment);
	template CORE_API FExpressionResult Evaluate<WIDECHAR>(const TArray<TCompiledToken<WIDECHAR>>& CompiledTokens, const TIOperatorEvaluationEnvironment<WIDECHAR>& InEnvironment);
}

template class TTokenStream<ANSICHAR>;
template class TTokenStream<UTF8CHAR>;
template class TTokenStream<WIDECHAR>;

template class TExpressionTokenConsumer<ANSICHAR>;
template class TExpressionTokenConsumer<UTF8CHAR>;
template class TExpressionTokenConsumer<WIDECHAR>;

template class TTokenDefinitions<ANSICHAR>;
template class TTokenDefinitions<WIDECHAR>;
template class TTokenDefinitions<UTF8CHAR>;

#if WITH_DEV_AUTOMATION_TESTS

namespace Tests
{
	struct FOperator {};

	struct FAnd { static const TCHAR* const Moniker; };
	struct FOr { static const TCHAR* const Moniker; };

	const TCHAR* const FAnd::Moniker = TEXT("&&");
	const TCHAR* const FOr::Moniker = TEXT("||");

	struct FMoveableType
	{
		static int32* LeakCount;

		FMoveableType(int32 InId)
			: Id(InId), bOwnsLeak(true)
		{
			++*LeakCount;
		}

		FMoveableType(FMoveableType&& In) : Id(-1), bOwnsLeak(false) { *this = MoveTemp(In); }
		FMoveableType& operator=(FMoveableType&& In)
		{
			if (bOwnsLeak)
			{
				bOwnsLeak = false;
				--*LeakCount;
			}

			Id = In.Id;
			In.Id = -1;
			
			bOwnsLeak = In.bOwnsLeak;

			In.bOwnsLeak = false;
			return *this;
		}

		FMoveableType(const FMoveableType& In) : Id(-1), bOwnsLeak(false) { *this = In; }
		const FMoveableType& operator=(const FMoveableType& In)
		{
			const bool bDidOwnLeak = bOwnsLeak;
			bOwnsLeak = In.bOwnsLeak;

			if (bOwnsLeak && !bDidOwnLeak)
			{
				++*LeakCount;
			}
			else if (!bOwnsLeak && bDidOwnLeak)
			{
				--*LeakCount;
			}
			return *this;
		}

		virtual ~FMoveableType()
		{
			if (bOwnsLeak)
			{
				--*LeakCount;
			}
		}

		int32 Id;
		bool bOwnsLeak;
	};
	
	int32* FMoveableType::LeakCount = nullptr;

	template<typename T>
	bool TestWithType(FAutomationTestBase* Test)
	{
		int32 NumLeaks = 0;
		
		// Test that move-assigning the expression node correctly assigns the data, and calls the destructors successfully
		{
			TGuardValue<int32*> LeakCounter(T::LeakCount, &NumLeaks);
			
			FExpressionNode Original(T(1));
			FExpressionNode New = MoveTemp(Original);
			
			int32 ResultingId = New.Cast<T>()->Id;
			if (ResultingId != 1)
			{
				Test->AddError(FString::Printf(TEXT("Expression node move operator did not operate correctly. Expected moved-to state to be 1, it's actually %d."), ResultingId));
				return false;
			}

			// Try assigning it over the top again
			Original = FExpressionNode(T(1));
			New = MoveTemp(Original);

			ResultingId = New.Cast<T>()->Id;
			if (ResultingId != 1)
			{
				Test->AddError(FString::Printf(TEXT("Expression node move operator did not operate correctly. Expected moved-to state to be 1, it's actually %d."), ResultingId));
				return false;
			}

			// Now try running it all through a parser
			FTokenDefinitions TokenDefs;
			FExpressionGrammar Grammar;
			FOperatorJumpTable JumpTable;

			// Only valid tokens are a, b, and +
			TokenDefs.DefineToken([](FExpressionTokenConsumer& Consumer){
				auto Token = Consumer.GetStream().GenerateToken(1);
				if (Token.IsSet())
				{
					switch(Consumer.GetStream().PeekChar())
					{
					case 'a': Consumer.Add(Token.GetValue(), T(1)); break;
					case '+': Consumer.Add(Token.GetValue(), FOperator()); break;
					}
				}
				return TOptional<FExpressionError>();
			});

			Grammar.DefinePreUnaryOperator<FOperator>();
			Grammar.DefineBinaryOperator<FOperator>(1);

			JumpTable.MapPreUnary<FOperator>([](const T& A)					{ return T(A.Id); });
			JumpTable.MapBinary<FOperator>([](const T& A, const T& B)		{ return T(A.Id); });

			ExpressionParser::Evaluate(TEXT("+a"), TokenDefs, Grammar, JumpTable);
			ExpressionParser::Evaluate(TEXT("a+a"), TokenDefs, Grammar, JumpTable);
			ExpressionParser::Evaluate(TEXT("+a++a"), TokenDefs, Grammar, JumpTable);
		}

		if (NumLeaks != 0)
		{
			Test->AddError(FString::Printf(TEXT("Expression node did not call wrapped type's destructors correctly. Potentially resulted in %d leaks."), NumLeaks));
			return false;
		}

		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FExpressionParserMoveableTypes, "System.Core.Expression Parser.Moveable Types", EAutomationTestFlags::EditorContext | EAutomationTestFlags::SmokeFilter)
	bool FExpressionParserMoveableTypes::RunTest( const FString& Parameters )
	{
		return TestWithType<FMoveableType>(this);
	}

	struct FHugeType : FMoveableType
	{
		FHugeType(int32 InId) : FMoveableType(InId) {}
		FHugeType(FHugeType&& In) : FMoveableType(MoveTemp(In)) {}
		FHugeType(const FHugeType& In) : FMoveableType(In) {}

		FHugeType& operator=(FHugeType&& In)
		{
			MoveTemp(In);
			return *this;
		}
		
		uint8 Padding[1024];
	};

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FExpressionParserAllocatedTypes, "System.Core.Expression Parser.Allocated Types", EAutomationTestFlags::EditorContext | EAutomationTestFlags::SmokeFilter)
	bool FExpressionParserAllocatedTypes::RunTest( const FString& Parameters )
	{
		return TestWithType<FHugeType>(this);
	}

}

DEFINE_EXPRESSION_NODE_TYPE(Tests::FMoveableType, 0xB7F3F127, 0xD5E74833, 0x9EAB754E, 0x6CF3AAC1)
DEFINE_EXPRESSION_NODE_TYPE(Tests::FHugeType, 0x4A329D81, 0x102343A8, 0xAB95BF45, 0x6578EE54)
DEFINE_EXPRESSION_NODE_TYPE(Tests::FOperator, 0xC777A5D7, 0x6895456C, 0x9854BFA0, 0xB71B5A8D)
DEFINE_EXPRESSION_NODE_TYPE(Tests::FAnd, 0x0687f9c5, 0xd8914cb0, 0xae52cc4c, 0x6770f520)
DEFINE_EXPRESSION_NODE_TYPE(Tests::FOr, 0x81e2b2a3, 0xbcf545d6, 0x95ae2eac, 0xcc5a9ba5)

struct FShortCircuitTestContext
{
	FShortCircuitTestContext() : NumOperatorsCalled(0) {}
	mutable int32 NumOperatorsCalled;
};

/** A basic math expression evaluator */
class FShortCircuitParser
{
public:
	/** Constructor that sets up the parser's lexer and compiler */
	FShortCircuitParser()
	{
		using namespace ExpressionParser;
		using namespace Tests;

		// A || !(B && C)
		TokenDefinitions.IgnoreWhitespace();
		TokenDefinitions.DefineToken([](FExpressionTokenConsumer& Consumer) -> TOptional<FExpressionError> {
			TOptional<FStringToken> Token = Consumer.GetStream().ParseToken(TEXT("true"));
			if (Token.IsSet())
			{
				Consumer.Add(Token.GetValue(), true);
			}

			Token = Consumer.GetStream().ParseToken(TEXT("false"));
			if (Token.IsSet())
			{
				Consumer.Add(Token.GetValue(), false);
			}

			return TOptional<FExpressionError>();
		});

		TokenDefinitions.DefineToken(&ConsumeSymbol<FSubExpressionStart>);
		TokenDefinitions.DefineToken(&ConsumeSymbol<FSubExpressionEnd>);
		TokenDefinitions.DefineToken(&ConsumeSymbol<FAnd>);
		TokenDefinitions.DefineToken(&ConsumeSymbol<FOr>);

		Grammar.DefineGrouping<FSubExpressionStart, FSubExpressionEnd>();

		bool bCanShortCircuit = true;
		Grammar.DefineBinaryOperator<FAnd>(1, EAssociativity::RightToLeft, bCanShortCircuit);
		Grammar.DefineBinaryOperator<FOr>(1, EAssociativity::RightToLeft, bCanShortCircuit);

		JumpTable.MapBinary<FAnd>([](bool A, bool B, const FShortCircuitTestContext* Context) {
			++Context->NumOperatorsCalled;
			return A && B;
		});

		JumpTable.MapBinary<FOr>([](bool A, bool B, const FShortCircuitTestContext* Context) {
			++Context->NumOperatorsCalled;
			return A || B;
		});

		JumpTable.MapShortCircuit<FOr>([](bool A) { return A; });
		JumpTable.MapShortCircuit<FAnd>([](bool A) { return !A; });
	}

	TValueOrError<bool, FExpressionError> Evaluate(const TCHAR* InExpression, const FShortCircuitTestContext& TestContext) const
	{
		using namespace ExpressionParser;

		TValueOrError<TArray<FExpressionToken>, FExpressionError> LexResult = ExpressionParser::Lex(InExpression, TokenDefinitions);
		if (!LexResult.IsValid())
		{
			return MakeError(LexResult.StealError());
		}

		TValueOrError<TArray<FCompiledToken>, FExpressionError> CompilationResult = ExpressionParser::Compile(LexResult.StealValue(), Grammar);
		if (!CompilationResult.IsValid())
		{
			return MakeError(CompilationResult.StealError());
		}

		TOperatorEvaluationEnvironment<FShortCircuitTestContext> Env(JumpTable, &TestContext);
		TValueOrError<FExpressionNode, FExpressionError> Result = ExpressionParser::Evaluate(CompilationResult.GetValue(), Env);
		if (!Result.IsValid())
		{
			return MakeError(Result.GetError());
		}

		if (const bool* Value = Result.GetValue().Cast<bool>())
		{
			return MakeValue(*Value);
		}

		return MakeError(NSLOCTEXT("Anon", "UnrecognizedResult", "Unrecognized result returned from expression"));
	}

	static FShortCircuitParser& Get()
	{
		static FShortCircuitParser Singleton;
		return Singleton;
	}

private:

	FTokenDefinitions TokenDefinitions;
	FExpressionGrammar Grammar;
	TOperatorJumpTable<FShortCircuitTestContext> JumpTable;

};

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FShortCircuitParserTest, "System.Core.Expression Parser.Short Circuit", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter | EAutomationTestFlags::HighPriority)
bool FShortCircuitParserTest::RunTest(const FString& Parameters)
{
	struct FExpectedResult
	{
		const TCHAR* const Expression;
		bool Result;
		int32 NumOperatorsCalled;
	};
	const FExpectedResult ExpectedResults[] = {
		FExpectedResult{ TEXT("true || (true && true)"), true, 0 },
		FExpectedResult{ TEXT("false && (true)"), false, 0 },
	};

	for (const FExpectedResult& Expected : ExpectedResults)
	{
		FShortCircuitTestContext Context;

		TValueOrError<bool, FExpressionError> Result = FShortCircuitParser::Get().Evaluate(Expected.Expression, Context);
		if (ensureAlways(Result.IsValid()))
		{
			ensureAlways(Expected.Result == Result.GetValue());
			ensureAlways(Expected.NumOperatorsCalled == Context.NumOperatorsCalled);
		}
	}

	return true;
}

namespace Tests
{
	template <class ResultTokenType>
	static TOptional<FExpressionError> ConsumeCharRangeGreedy(FExpressionTokenConsumer& Consumer, TCHAR Start, TCHAR EndInclusive)
	{
		TOptional<FStringToken> Token = Consumer.GetStream().ParseToken([=](TCHAR Ch)
		{
			return Ch >= Start && Ch <= EndInclusive ? EParseState::Continue : EParseState::StopBefore;
		});
		if (Token)
		{
			check(Token->IsValid());
			Consumer.Add(*Token, ResultTokenType());
		}
		return {};
	}

	struct FLowerAlphaTokenTag {};
	static TOptional<FExpressionError> ConsumeLowerAlphaGreedy(FExpressionTokenConsumer& Consumer)
	{
		return ConsumeCharRangeGreedy<FLowerAlphaTokenTag>(Consumer, 'a', 'z');
	}

	struct FDigitsTokenTag {};
	static TOptional<FExpressionError> ConsumeDigitsGreedy(FExpressionTokenConsumer& Consumer)
	{
		return ConsumeCharRangeGreedy<FDigitsTokenTag>(Consumer, '0', '9');
	}

	struct FStringTokenTag {};
} // end namespace Tests

DEFINE_EXPRESSION_NODE_TYPE(Tests::FLowerAlphaTokenTag, 0x01772467, 0xb30c4b0f, 0xb7863e3f, 0x5a52360b);
DEFINE_EXPRESSION_NODE_TYPE(Tests::FDigitsTokenTag, 0x51e2dea9, 0xa09247ec, 0x93651811, 0x8f4df950);
DEFINE_EXPRESSION_NODE_TYPE(Tests::FStringTokenTag, 0x3365d89e, 0xa4344abe, 0xbcf04bdc, 0xdcd728c8);

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FBasicLexerTest, "System.Core.Expression Parser.Lexer.Basic", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter | EAutomationTestFlags::HighPriority)
bool FBasicLexerTest::RunTest(const FString& Parameters)
{
	FTokenDefinitions TokenDefinitions;
	TokenDefinitions.DefineToken(Tests::ConsumeLowerAlphaGreedy);

	ExpressionParser::LexResultType Result = ExpressionParser::Lex(TEXT("abc"), TokenDefinitions);
	UTEST_TRUE(TEXT("Lex succeeded"), Result.HasValue());
	UTEST_EQUAL(TEXT("Found one token"), Result.GetValue().Num(), 1);
	UTEST_EQUAL(TEXT("Token contains expected 3 characters"), Result.GetValue()[0].Context.GetString().Len(), 3);

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FLexerIgnoreWhitespaceTest, "System.Core.Expression Parser.Lexer.Ignore whitespace", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter | EAutomationTestFlags::HighPriority)
bool FLexerIgnoreWhitespaceTest::RunTest(const FString& Parameters)
{
	FTokenDefinitions TokenDefinitions;
	TokenDefinitions.DefineToken(Tests::ConsumeLowerAlphaGreedy);
	TokenDefinitions.DefineToken(Tests::ConsumeDigitsGreedy);
	TokenDefinitions.IgnoreWhitespace();

	ExpressionParser::LexResultType Result = ExpressionParser::Lex(TEXT("abc 123"), TokenDefinitions);
	UTEST_TRUE(TEXT("Lex succeeded"), Result.HasValue());
	UTEST_EQUAL(TEXT("Found two tokens"), Result.GetValue().Num(), 2);
	UTEST_EQUAL(TEXT("First token contains expected 3 characters"), Result.GetValue()[0].Context.GetString().Len(), 3);
	UTEST_EQUAL(TEXT("Second token contains expected 3 characters"), Result.GetValue()[1].Context.GetString().Len(), 3);

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FLexerParseStringTest, "System.Core.Expression Parser.Lexer.Parse string", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter | EAutomationTestFlags::HighPriority)
bool FLexerParseStringTest::RunTest(const FString& Parameters)
{
	// an example of matching but ignoring stuff before and after a token

	auto ConsumeString = [](FExpressionTokenConsumer& Consumer) -> TOptional<FExpressionError> {

		FTokenStream& Stream = Consumer.GetStream();
		if (TOptional<FStringToken> InitialQuoteToken = Stream.ParseSymbol('"'))
		{
			TOptional<FStringToken> ContentsToken = Stream.ParseToken([](TCHAR C)
			{
				return C == '"' ? EParseState::StopBefore : EParseState::Continue;
			}, &*InitialQuoteToken);

			// note: continue to accumulate to initial token, leaving contents token untouched
			TOptional<FStringToken> CloseTokenToIgnore = Stream.ParseSymbol('"', &*InitialQuoteToken);
			if (!ContentsToken || !CloseTokenToIgnore)
			{
				return FExpressionError(FText::AsCultureInvariant(TEXT("Unterminated string")));
			}
			Consumer.Add(*ContentsToken, Tests::FStringTokenTag());
			// skip close quote
			Stream.SetReadPos(*CloseTokenToIgnore);
		}
		return {};
	};

	FTokenDefinitions TokenDefinitions;
	TokenDefinitions.DefineToken(ConsumeString);
	TokenDefinitions.DefineToken(Tests::ConsumeDigitsGreedy);
	TokenDefinitions.IgnoreWhitespace();

	ExpressionParser::LexResultType Result = ExpressionParser::Lex(TEXT("\"1 a\" 123"), TokenDefinitions);
	UTEST_TRUE(TEXT("Lex succeeded"), Result.HasValue());
	UTEST_EQUAL(TEXT("Found two tokens"), Result.GetValue().Num(), 2);
	UTEST_EQUAL(TEXT("First token contains expected 3 characters"), Result.GetValue()[0].Context.GetString().Len(), 3);
	UTEST_EQUAL(TEXT("Second token contains expected 3 characters"), Result.GetValue()[1].Context.GetString().Len(), 3);

	ExpressionParser::LexResultType ExpectedFailureResult = ExpressionParser::Lex(TEXT("\"abc"), TokenDefinitions);
	UTEST_TRUE(TEXT("Unterminated string"), ExpectedFailureResult.HasError());

	ExpectedFailureResult = ExpressionParser::Lex(TEXT("\""), TokenDefinitions);
	UTEST_TRUE(TEXT("Just quote"), ExpectedFailureResult.HasError());

	return true;
}

#endif // WITH_DEV_AUTOMATION_TESTS

#undef LOCTEXT_NAMESPACE
