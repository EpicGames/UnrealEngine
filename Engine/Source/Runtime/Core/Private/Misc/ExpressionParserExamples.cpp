// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "Misc/Optional.h"
#include "Internationalization/Internationalization.h"
#include "Templates/ValueOrError.h"
#include "Misc/ExpressionParserTypes.h"
#include "Misc/ExpressionParser.h"
#include "Math/BasicMathExpressionEvaluator.h"

// Don't include this code in shipping or perf test builds
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

#define LOCTEXT_NAMESPACE "ExpressionParserExamples"

class FExampleMathEvaluator
{
public:

	struct FExampleAdd {};

	/** Very simple math expression parser that supports addition of numbers */
	FExampleMathEvaluator()
	{
		using namespace ExpressionParser;

		TokenDefinitions.IgnoreWhitespace();

		// Define a tokenizer that will tokenize numbers in the expression
		TokenDefinitions.DefineToken(&ConsumeNumber);

		// Define a tokenizer for the arithmetic add
		TokenDefinitions.DefineToken([](FExpressionTokenConsumer& Consumer){

			// Called at the start of every new token - see if it's a + character
			TOptional<FStringToken> Token = Consumer.GetStream().ParseSymbol(TEXT('+'));
			if (Token.IsSet())
			{
				// Add the token to the consumer. This moves the read position in the stream to the end of the token.
				Consumer.Add(Token.GetValue(), FExampleAdd());
			}

			// Optionally return an error if there is a good reason to
			return TOptional<FExpressionError>();
		});


		// Define our FExampleAdd token as a binary operator
		Grammar.DefineBinaryOperator<FExampleAdd>(5);
		// And provide an operator overload for it
		JumpTable.MapBinary<FExampleAdd>([](double A, double B) { return A + B;	});
	}

	/** Evaluate the given expression, resulting in either a double value, or an error */
	TValueOrError<double, FExpressionError> Evaluate(const TCHAR* InExpression) const
	{
		TValueOrError<FExpressionNode, FExpressionError> Result = ExpressionParser::Evaluate(InExpression, TokenDefinitions, Grammar, JumpTable);
		if (!Result.IsValid())
		{
			return MakeError(Result.GetError());
		}
		
		if (const double* Numeric = Result.GetValue().Cast<double>())
		{
			return MakeValue(*Numeric);
		}
		else
		{
			return MakeError(LOCTEXT("UnrecognizedResult", "Unrecognized result returned from expression"));
		}
	}
	
private:
	FTokenDefinitions TokenDefinitions;
	FExpressionGrammar Grammar;
	FOperatorJumpTable JumpTable;
};
DEFINE_EXPRESSION_NODE_TYPE(FExampleMathEvaluator::FExampleAdd, 0x4BF093DC, 0xA92247B5, 0xAE53A9B3, 0x64A84D2C)

#undef LOCTEXT_NAMESPACE

#endif //!(UE_BUILD_SHIPPING || UE_BUILD_TEST)
