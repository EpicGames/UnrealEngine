// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/FbxErrors.h"

#include "CoreTypes.h"
	
/** Generic */
FLazyName FFbxErrors::Generic_ImportingNewObjectFailed(TEXT("Generic_ImportingNewObjectFailed"));

FLazyName FFbxErrors::Generic_ReimportingObjectFailed(TEXT("Generic_ReimportingObjectFailed"));

FLazyName FFbxErrors::Generic_LoadingSceneFailed(TEXT("Generic_LoadingSceneFailed"));

FLazyName FFbxErrors::Generic_InvalidCharacterInName(TEXT("Generic_InvalidCharacterInName"));

FLazyName FFbxErrors::Generic_SameNameAssetExists(TEXT("Generic_SameNameAssetExists"));

FLazyName FFbxErrors::Generic_SameNameAssetOverriding(TEXT("Generic_SameNameAssetOverriding"));

FLazyName FFbxErrors::Generic_CannotDeleteReferenced(TEXT("Generic_CannotDeleteReferenced"));

FLazyName FFbxErrors::Generic_FBXFileParseFailed(TEXT("Generic_FBXFileParseFailed"));

FLazyName FFbxErrors::Generic_MeshNotFound(TEXT("Generic_MeshNotFound"));

FLazyName FFbxErrors::Generic_CannotDetectImportType(TEXT("Generic_CannotDetectImportType"));

/** Mesh Generic **/

FLazyName FFbxErrors::Generic_Mesh_NoGeometry(TEXT("Generic_Mesh_NoGeometry"));

FLazyName FFbxErrors::Generic_Mesh_SmallGeometry(TEXT("Generic_Mesh_SmallGeometry"));

FLazyName FFbxErrors::Generic_Mesh_TriangulationFailed(TEXT("Generic_Mesh_TriangulationFailed"));

FLazyName FFbxErrors::Generic_Mesh_ConvertSmoothingGroupFailed(TEXT("Generic_Mesh_ConvertSmoothingGroupFailed"));

FLazyName FFbxErrors::Generic_Mesh_UnsupportingSmoothingGroup(TEXT("Generic_Mesh_UnsupportingSmoothingGroup"));

FLazyName FFbxErrors::Generic_Mesh_MaterialIndexInconsistency(TEXT("Generic_Mesh_MaterialIndexInconsistency"));

FLazyName FFbxErrors::Generic_Mesh_MeshNotFound(TEXT("Generic_Mesh_MeshNotFound"));

FLazyName FFbxErrors::Generic_Mesh_NoSmoothingGroup(TEXT("Generic_Mesh_NoSmoothingGroup"));

FLazyName FFbxErrors::Generic_Mesh_LOD_InvalidIndex(TEXT("Generic_Mesh_LOD_InvalidIndex"));

FLazyName FFbxErrors::Generic_Mesh_LOD_NoFileSelected(TEXT("Generic_Mesh_LOD_NoFileSelected"));

FLazyName FFbxErrors::Generic_Mesh_LOD_MultipleFilesSelected(TEXT("Generic_Mesh_LOD_MultipleFilesSelected"));

FLazyName FFbxErrors::Generic_Mesh_SkinxxNameError(TEXT("Generic_Mesh_SkinxxNameError"));

FLazyName FFbxErrors::Generic_Mesh_TooManyLODs(TEXT("Generic_Mesh_TooManyLODs"));

FLazyName FFbxErrors::Generic_Mesh_TangentsComputeError(TEXT("Generic_Mesh_TangentsComputeError"));

FLazyName FFbxErrors::Generic_Mesh_NoReductionModuleAvailable(TEXT("Generic_Mesh_NoReductionModuleAvailable"));

FLazyName FFbxErrors::Generic_Mesh_TooMuchUVChannels(TEXT("Generic_Mesh_TooMuchUVChannels"));

/** Static Mesh **/
FLazyName FFbxErrors::StaticMesh_TooManyMaterials(TEXT("StaticMesh_TooManyMaterials"));

FLazyName FFbxErrors::StaticMesh_UVSetLayoutProblem(TEXT("StaticMesh_UVSetLayoutProblem"));

FLazyName FFbxErrors::StaticMesh_NoTriangles(TEXT("StaticMesh_NoTriangles"));

FLazyName FFbxErrors::StaticMesh_BuildError(TEXT("StaticMesh_BuildError"));

FLazyName FFbxErrors::StaticMesh_AllTrianglesDegenerate(TEXT("StaticMesh_AllTrianglesDegenerate"));

FLazyName FFbxErrors::StaticMesh_AdjacencyOptionForced(TEXT("StaticMesh_AdjacencyOptionForced"));

/** SkeletalMesh **/
FLazyName FFbxErrors::SkeletalMesh_DifferentRoots(TEXT("SkeletalMesh_DifferentRoot"));

FLazyName FFbxErrors::SkeletalMesh_DuplicateBones(TEXT("SkeletalMesh_DuplicateBones"));

FLazyName FFbxErrors::SkeletalMesh_NoInfluences(TEXT("SkeletalMesh_NoInfluences"));

FLazyName FFbxErrors::SkeletalMesh_TooManyInfluences(TEXT("SkeletalMesh_TooManyInfluences"));

FLazyName FFbxErrors::SkeletalMesh_RestoreSortingMismatchedStrips(TEXT("SkeletalMesh_RestoreSortingMismatchedStrips"));

FLazyName FFbxErrors::SkeletalMesh_RestoreSortingNoSectionMatch(TEXT("SkeletalMesh_RestoreSortingNoSectionMatch"));

FLazyName FFbxErrors::SkeletalMesh_RestoreSortingForSectionNumber(TEXT("SkeletalMesh_RestoreSortingForSectionNumber"));

FLazyName FFbxErrors::SkeletalMesh_NoMeshFoundOnRoot(TEXT("SkeletalMesh_NoMeshFoundOnRoot"));

FLazyName FFbxErrors::SkeletalMesh_InvalidRoot(TEXT("SkeletalMesh_InvalidRoot"));

FLazyName FFbxErrors::SkeletalMesh_InvalidBone(TEXT("SkeletalMesh_InvalidBone"));

FLazyName FFbxErrors::SkeletalMesh_InvalidNode(TEXT("SkeletalMesh_InvalidNode"));

FLazyName FFbxErrors::SkeletalMesh_NoWeightsOnDeformer(TEXT("SkeletalMesh_NoWeightsOnDeformer"));

FLazyName FFbxErrors::SkeletalMesh_NoBindPoseInScene(TEXT("SkeletalMesh_NoBindPoseInScene"));

FLazyName FFbxErrors::SkeletalMesh_NoAssociatedCluster(TEXT("SkeletalMesh_NoAssociatedCluster"));

FLazyName FFbxErrors::SkeletalMesh_NoBoneFound(TEXT("SkeletalMesh_NoBoneFound"));

FLazyName FFbxErrors::SkeletalMesh_InvalidBindPose(TEXT("SkeletalMesh_InvalidBindPose"));

FLazyName FFbxErrors::SkeletalMesh_MultipleRoots(TEXT("SkeletalMesh_MultipleRoots"));

FLazyName FFbxErrors::SkeletalMesh_BonesAreMissingFromBindPose(TEXT("SkeletalMesh_BonesAreMissingFromBindPose"));

FLazyName FFbxErrors::SkeletalMesh_VertMissingInfluences(TEXT("SkeletalMesh_VertMissingInfluences"));

FLazyName FFbxErrors::SkeletalMesh_SectionWithNoTriangle(TEXT("SkeletalMesh_SectionWithNoTriangle"));

FLazyName FFbxErrors::SkeletalMesh_TooManyVertices(TEXT("SkeletalMesh_TooManyVertices"));

FLazyName FFbxErrors::SkeletalMesh_FailedToCreatePhyscisAsset(TEXT("SkeletalMesh_FailedToCreatePhyscisAsset"));

FLazyName FFbxErrors::SkeletalMesh_SkeletonRecreateError(TEXT("SkeletalMesh_SkeletonRecreateError"));

FLazyName FFbxErrors::SkeletalMesh_ExceedsMaxBoneCount(TEXT("SkeletalMesh_ExceedsMaxBoneCount"));

FLazyName FFbxErrors::SkeletalMesh_NoUVSet(TEXT("SkeletalMesh_NoUVSet"));

FLazyName FFbxErrors::SkeletalMesh_LOD_MissingBone(TEXT("SkeletalMesh_LOD_MissingBone"));

FLazyName FFbxErrors::SkeletalMesh_LOD_FailedToImport(TEXT("SkeletalMesh_LOD_FailedToImport"));

FLazyName FFbxErrors::SkeletalMesh_LOD_RootNameIncorrect(TEXT("SkeletalMesh_LOD_RootNameIncorrect"));

FLazyName FFbxErrors::SkeletalMesh_LOD_BonesDoNotMatch(TEXT("SkeletalMesh_LOD_BonesDoNotMatch"));

FLazyName FFbxErrors::SkeletalMesh_LOD_IncorrectParent(TEXT("SkeletalMesh_LOD_IncorrectParent"));

FLazyName FFbxErrors::SkeletalMesh_LOD_HasSoftVerts(TEXT("SkeletalMesh_LOD_HasSoftVerts"));

FLazyName FFbxErrors::SkeletalMesh_LOD_MissingSocketBone(TEXT("SkeletalMesh_LOD_MissingSocketBone"));

FLazyName FFbxErrors::SkeletalMesh_LOD_MissingMorphTarget(TEXT("SkeletalMesh_LOD_MissingMorphTarget"));

FLazyName FFbxErrors::SkeletalMesh_FillImportDataFailed(TEXT("SkeletalMesh_FillImportDataFailed"));

FLazyName FFbxErrors::SkeletalMesh_InvalidPosition(TEXT("SkeletalMesh_InvalidPosition"));

FLazyName FFbxErrors::SkeletalMesh_AttributeComponentCountMismatch(TEXT("SkeletalMesh_AttributeComponentCountMismatch"));

/** Animation **/
FLazyName FFbxErrors::Animation_CouldNotFindRootTrack(TEXT("Animation_CouldNotFindRootTrack"));

FLazyName FFbxErrors::Animation_CouldNotBuildSkeleton(TEXT("Animation_CouldNotBuildSkeleton"));

FLazyName FFbxErrors::Animation_CouldNotFindTrack(TEXT("Animation_CouldNotFindTrack"));

FLazyName FFbxErrors::Animation_ZeroLength(TEXT("Animation_ZeroLength"));

FLazyName FFbxErrors::Animation_RootTrackMismatch(TEXT("Animation_RootTrackMismatch"));

FLazyName FFbxErrors::Animation_DuplicatedBone(TEXT("Animation_DuplicatedBone"));

FLazyName FFbxErrors::Animation_MissingBones(TEXT("Animation_MissingBones"));

FLazyName FFbxErrors::Animation_InvalidData(TEXT("Animation_InvalidData"));

FLazyName FFbxErrors::Animation_TransformError(TEXT("Animation_TransformError"));

FLazyName FFbxErrors::Animation_DifferentLength(TEXT("Animation_DifferentLength"));

FLazyName FFbxErrors::Animation_CurveNotFound(TEXT("Animation_CurveNotFound"));

namespace
{
	const TCHAR* FbxErrorsURLPath = TEXT("WorkingWithContent/Importing/FBX/FbxErrors/");
	const TCHAR* FbxErrorsLocalPath = TEXT("Shared/Editor/FbxErrors");
}

FFbxErrorToken::FFbxErrorToken(const FName& InErrorName)
	: FDocumentationToken(FbxErrorsURLPath, FbxErrorsLocalPath, InErrorName.ToString())
{
}
