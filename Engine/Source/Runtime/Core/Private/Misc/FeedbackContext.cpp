// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/FeedbackContext.h"

#include "CoreTypes.h"
#include "HAL/PlatformTime.h"
#include "Logging/StructuredLog.h"
#include "Misc/App.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CoreMisc.h"
#include "Misc/OutputDeviceConsole.h"
#include "Misc/OutputDeviceHelper.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Misc/SlowTask.h"
#include "Misc/StringBuilder.h"

FFeedbackContext::FFeedbackContext() = default;

FFeedbackContext::~FFeedbackContext()
{
	ensureMsgf(LegacyAPIScopes.Num() == 0, TEXT("EndSlowTask has not been called for %d outstanding tasks"), LegacyAPIScopes.Num());
}

void FFeedbackContext::Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const FName& Category)
{
	Verbosity = ResolveVerbosity(Verbosity);
	if (IsRunningCommandlet())
	{
		if (Verbosity == ELogVerbosity::Error || Verbosity == ELogVerbosity::Warning)
		{
			AddToHistory(V, Verbosity, Category, -1.0);
		}
		if (GLogConsole && !GLog->IsRedirectingTo(GLogConsole))
		{
			GLogConsole->Serialize(V, Verbosity, Category);
		}
	}
	if (!GLog->IsRedirectingTo(this))
	{
		GLog->Serialize(V, Verbosity, Category);
	}
}

void FFeedbackContext::Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const FName& Category, double Time)
{
	Verbosity = ResolveVerbosity(Verbosity);
	if (IsRunningCommandlet())
	{
		if (Verbosity == ELogVerbosity::Error || Verbosity == ELogVerbosity::Warning)
		{
			AddToHistory(V, Verbosity, Category, Time);
		}
		if (GLogConsole && !GLog->IsRedirectingTo(GLogConsole))
		{
			GLogConsole->Serialize(V, Verbosity, Category, Time);
		}
	}
	if (!GLog->IsRedirectingTo(this))
	{
		GLog->Serialize(V, Verbosity, Category, Time);
	}
}

void FFeedbackContext::SerializeRecord(const UE::FLogRecord& Record)
{
	ELogVerbosity::Type Verbosity = ResolveVerbosity(Record.GetVerbosity());
	TOptional<UE::FLogRecord> RecordOverride;
	if (Record.GetVerbosity() != Verbosity)
	{
		RecordOverride = Record;
		RecordOverride->SetVerbosity(Verbosity);
	}
	if (IsRunningCommandlet())
	{
		if (Verbosity == ELogVerbosity::Error || Verbosity == ELogVerbosity::Warning)
		{
			AddRecordToHistory(RecordOverride.Get(Record));
		}
		if (GLogConsole && !GLog->IsRedirectingTo(GLogConsole))
		{
			GLogConsole->SerializeRecord(RecordOverride.Get(Record));
		}
	}
	if (!GLog->IsRedirectingTo(this))
	{
		GLog->SerializeRecord(RecordOverride.Get(Record));
	}
}

ELogVerbosity::Type FFeedbackContext::ResolveVerbosity(ELogVerbosity::Type Verbosity) const
{
	if (Verbosity == ELogVerbosity::Error && TreatErrorsAsWarnings)
	{
		// Only override the verbosity of errors if we are not currently handling a fatal failure.
		// Some of the codepaths invoked when handling critical/fatal errors might involve logging
		// the failure as an error along with some additional information, and we don't want to
		// suppress fatal failures.
		if (!GIsCriticalError)
		{
			return ELogVerbosity::Warning;
		}
	}
	if (Verbosity == ELogVerbosity::Warning && TreatWarningsAsErrors)
	{
		return ELogVerbosity::Error;
	}
	return Verbosity;
}

void FFeedbackContext::FormatLine(FStringBuilderBase& Out, const TCHAR* V, ELogVerbosity::Type Verbosity, const FName& Category, double Time, ELogVerbosity::Type* OutVerbosity) const
{
	Verbosity = ResolveVerbosity(Verbosity);
	if (OutVerbosity)
	{
		*OutVerbosity = Verbosity;
	}
	if (FContextSupplier* Context = GetContext())
	{
		Out.Append(Context->GetContext()).Append(TEXTVIEW(" : "));
	}
	FOutputDeviceHelper::AppendFormatLogLine(Out, Verbosity, Category, V);
}

void FFeedbackContext::FormatRecordLine(FStringBuilderBase& Out, const UE::FLogRecord& Record, ELogVerbosity::Type* OutVerbosity) const
{
	ELogVerbosity::Type Verbosity = ResolveVerbosity(Record.GetVerbosity());
	if (OutVerbosity)
	{
		*OutVerbosity = Verbosity;
	}
	if (FContextSupplier* Context = GetContext())
	{
		Out.Append(Context->GetContext()).Append(TEXTVIEW(" : "));
	}
	FOutputDeviceHelper::AppendFormatLogLine(Out, Verbosity, Record.GetCategory());
	Record.FormatMessageTo(Out);
}

void FFeedbackContext::AddToHistory(const TCHAR* V, ELogVerbosity::Type Verbosity, const FName& Category, double Time)
{
	if (GIsCriticalError)
	{
		// Turn off the storing of Errors and Warnings when GIsCriticalError is true.
		// In a critical error (e.g. crash handling), it might be the case that Malloc is returning bad data,
		// so we want to avoid dynamic allocation (such as the construction of strings below) as much as possible.
		return;
	}
	TStringBuilder<512> Line;
	FormatLine(Line, V, Verbosity, Category, Time, &Verbosity);
	if (Verbosity == ELogVerbosity::Error)
	{
		AddError(FString(Line));
	}
	else
	{
		AddWarning(FString(Line));
	}
}

void FFeedbackContext::AddRecordToHistory(const UE::FLogRecord& Record)
{
	TStringBuilder<512> Line;
	ELogVerbosity::Type Verbosity;
	FormatRecordLine(Line, Record, &Verbosity);
	if (Verbosity == ELogVerbosity::Warning)
	{
		AddWarning(FString(Line));
	}
	else if (Verbosity == ELogVerbosity::Error)
	{
		AddError(FString(Line));
	}
}

bool FFeedbackContext::YesNof(const FText& Question)
{
	if (!GIsSilent && !FApp::IsUnattended())
	{
		FPlatformMisc::LowLevelOutputDebugString(*Question.ToString());
	}
	return false;
}

void FFeedbackContext::RequestUpdateUI(bool bForceUpdate)
{
	// Only update a maximum of 5 times a second
	static double MinUpdateTimeS = 0.2;

	static double LastUIUpdateTime = FPlatformTime::Seconds();
	const double CurrentTime = FPlatformTime::Seconds();

	if (bForceUpdate || CurrentTime - LastUIUpdateTime > MinUpdateTimeS)
	{
		LastUIUpdateTime = CurrentTime;
		UpdateUI();
	}
}

void FFeedbackContext::UpdateUI()
{
	ensure(IsInGameThread());

	if (ScopeStack.Num() != 0)
	{
		ProgressReported(ScopeStack.GetProgressFraction(0), ScopeStack[0]->GetCurrentMessage());
	}
}

/**** Begin legacy API ****/
void FFeedbackContext::BeginSlowTask( const FText& Task, bool ShowProgressDialog, bool bShowCancelButton )
{
	ensure(IsInGameThread());

	TUniquePtr<FSlowTask> NewScope(new FSlowTask(0, Task, true, *this));
	if (ShowProgressDialog)
	{
		NewScope->MakeDialogDelayed(3.0f, bShowCancelButton);
	}

	NewScope->Initialize();
	LegacyAPIScopes.Add(MoveTemp(NewScope));
}

void FFeedbackContext::UpdateProgress( int32 Numerator, int32 Denominator )
{
	ensure(IsInGameThread());

	if (LegacyAPIScopes.Num() != 0)
	{
		LegacyAPIScopes.Last()->TotalAmountOfWork = (float)Denominator;
		LegacyAPIScopes.Last()->CompletedWork = (float)Numerator;
		LegacyAPIScopes.Last()->CurrentFrameScope = (float)(Denominator - Numerator);
		LegacyAPIScopes.Last()->TickProgress();
	}
}

void FFeedbackContext::StatusUpdate( int32 Numerator, int32 Denominator, const FText& StatusText )
{
	ensure(IsInGameThread());

	if (LegacyAPIScopes.Num() != 0)
	{
		if (Numerator > 0 && Denominator > 0)
		{
			UpdateProgress(Numerator, Denominator);
		}
		LegacyAPIScopes.Last()->FrameMessage = StatusText;
		RequestUpdateUI();
	}
}

void FFeedbackContext::StatusForceUpdate( int32 Numerator, int32 Denominator, const FText& StatusText )
{
	ensure(IsInGameThread());

	if (LegacyAPIScopes.Num() != 0)
	{
		UpdateProgress(Numerator, Denominator);
		LegacyAPIScopes.Last()->FrameMessage = StatusText;
		UpdateUI();
	}
}

void FFeedbackContext::EndSlowTask()
{
	ensure(IsInGameThread());

	check(LegacyAPIScopes.Num() != 0);
	LegacyAPIScopes.Last()->Destroy();
	LegacyAPIScopes.Pop();
}
/**** End legacy API ****/

bool FFeedbackContext::IsPlayingInEditor() const
{
	return GIsPlayInEditorWorld;
}
