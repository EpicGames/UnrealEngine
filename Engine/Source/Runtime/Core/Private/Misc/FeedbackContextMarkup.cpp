// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/FeedbackContextMarkup.h"

#include "Containers/UnrealString.h"
#include "HAL/PlatformProcess.h"
#include "Internationalization/Text.h"
#include "Logging/LogVerbosity.h"
#include "Math/NumericLimits.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CString.h"
#include "Misc/Char.h"
#include "Misc/FeedbackContext.h"

bool FFeedbackContextMarkup::ParseCommand(const FString& Line, FFeedbackContext* Warn)
{
	const TCHAR *Text = *Line;
	if(ReadToken(Text, TEXT("@progress")))
	{
		FString Status;
		bool bHaveStatus = ReadString(Text, Status);

		int32 Numerator, Denominator;
		bool bHaveProgress = ReadProgress(Text, Numerator, Denominator);

		if(*Text == 0)
		{
			if(bHaveProgress && bHaveStatus)
			{
				Warn->StatusUpdate(Numerator, Denominator, FText::FromString(Status));
				return true;
			}
			if(bHaveProgress)
			{
				Warn->UpdateProgress(Numerator, Denominator);
				return true;
			}
		}
	}
	return false;
}

bool FFeedbackContextMarkup::PipeProcessOutput(const FText& Description, const FString& URL, const FString& Params, FFeedbackContext* Warn, int32* OutExitCode)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("FFeedbackContextMarkup::PipeProcessOutput");
	bool bRes;

	// Create a read and write pipe for the child process
	void* PipeRead = NULL;
	void* PipeWrite = NULL;
	verify(FPlatformProcess::CreatePipe(PipeRead, PipeWrite));

	// Start the slow task
	Warn->BeginSlowTask(Description, true, true);

	// Create the process
	FProcHandle ProcessHandle = FPlatformProcess::CreateProc(*URL, *Params, false, true, true, NULL, 0, NULL, PipeWrite);
	if(ProcessHandle.IsValid())
	{
		FString BufferedText;
		for(bool bProcessFinished = false; !bProcessFinished; )
		{
			bProcessFinished = FPlatformProcess::GetProcReturnCode(ProcessHandle, OutExitCode);
		
			if(!bProcessFinished && Warn->ReceivedUserCancel())
			{
				FPlatformProcess::TerminateProc(ProcessHandle);
				bProcessFinished = true;
			}

			BufferedText += FPlatformProcess::ReadPipe(PipeRead);

			int32 EndOfLineIdx;
			while(BufferedText.FindChar(TEXT('\n'), EndOfLineIdx))
			{
				FString Line = BufferedText.Left(EndOfLineIdx);
				Line.RemoveFromEnd(TEXT("\r"), ESearchCase::CaseSensitive);

				if(!ParseCommand(Line, Warn))
				{
					Warn->Log(*Line);
				}

				BufferedText.MidInline(EndOfLineIdx + 1, MAX_int32, EAllowShrinking::No);
			}

			FPlatformProcess::Sleep(0.1f);
		}
		ProcessHandle.Reset();
		bRes = true;
	}
	else
	{
		Warn->Logf(ELogVerbosity::Error, TEXT("Couldn't create process '%s'"), *URL);
		bRes = false;
	}

	// Finish the slow task
	Warn->EndSlowTask();

	// Close the pipes
	FPlatformProcess::ClosePipe(PipeRead, PipeWrite);
	return bRes;
}

bool FFeedbackContextMarkup::ReadToken(const TCHAR*& Text, const TCHAR* Token)
{
	int32 Length = FCString::Strlen(Token);
	if(FCString::Strncmp(Text, Token, Length) == 0)
	{
		if(Text[Length] == 0 || FChar::IsWhitespace(Text[Length]))
		{
			Text = SkipWhitespace(Text + Length);
			return true;
		}
	}
	return false;
}

bool FFeedbackContextMarkup::ReadProgress(const TCHAR*& Text, int32& OutNumerator, int32& OutDenominator)
{
	uint32 FirstInteger;
	if(ReadInteger(Text, FirstInteger))
	{
		if(*Text == '%')
		{
			OutNumerator = FirstInteger;
			OutDenominator = 100;
			Text = SkipWhitespace(Text + 1);
			return true;
		}
		else if(*Text == '/')
		{
			Text++;

			uint32 SecondInteger;
			if(ReadInteger(Text, SecondInteger))
			{
				OutNumerator = FirstInteger;
				OutDenominator = SecondInteger;
				Text = SkipWhitespace(Text);
				return true;
			}
		}
	}
	return false;
}

bool FFeedbackContextMarkup::ReadInteger(const TCHAR*& Text, uint32& OutInteger)
{
	if(FChar::IsDigit(*Text))
	{
		TCHAR *End;
		OutInteger = (uint32)FCString::Strtoui64(Text, &End, 10);
		Text = End;
		while(FChar::IsWhitespace(*Text)) Text++;
		return true;
	}
	return false;
}

bool FFeedbackContextMarkup::ReadString(const TCHAR*& Text, FString& OutString)
{
	if(*Text == '\'' || *Text == '\"')
	{
		for(const TCHAR *End = Text + 1; *End; End++)
		{
			if(*End == *Text)
			{
				OutString = FString::ConstructFromPtrSize(Text + 1, UE_PTRDIFF_TO_INT32(End - (Text + 1)));
				do { End++; } while(FChar::IsWhitespace(*End));
				Text = End;
				return true;
			}
		}
	}
	return false;
}

const TCHAR *FFeedbackContextMarkup::SkipWhitespace(const TCHAR* Text)
{
	while(FChar::IsWhitespace(*Text))
	{
		Text++;
	}
	return Text;
}
