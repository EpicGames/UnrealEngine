// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/FileHelper.h"

#include "Async/AsyncFileHandle.h"
#include "AutoRTFM.h"
#include "Containers/StringConv.h"
#include "Logging/LogMacros.h"
#include "CoreGlobals.h"
#include "HAL/PlatformMath.h"
#include "Math/NumericLimits.h"
#include "Memory/MemoryView.h"
#include "Memory/SharedBuffer.h"
#include "Misc/ByteSwap.h"
#include "Misc/CoreMisc.h"
#include "Misc/Paths.h"
#include "Math/IntRect.h"
#include "Misc/OutputDeviceFile.h"
#include "Misc/StringBuilder.h"
#include "ProfilingDebugging/ProfilingHelpers.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/SecureHash.h"
#include "HAL/FileManagerGeneric.h"
#include "Tasks/Pipe.h"
#include "Templates/Function.h"
#include "Templates/UniquePtr.h"

#define LOCTEXT_NAMESPACE "FileHelper"

static const TCHAR* InvalidFilenames[] = {
	TEXT("CON"), TEXT("PRN"), TEXT("AUX"), TEXT("CLOCK$"), TEXT("NUL"), TEXT("NONE"),
	TEXT("COM1"), TEXT("COM2"), TEXT("COM3"), TEXT("COM4"), TEXT("COM5"), TEXT("COM6"), TEXT("COM7"), TEXT("COM8"), TEXT("COM9"),
	TEXT("LPT1"), TEXT("LPT2"), TEXT("LPT3"), TEXT("LPT4"), TEXT("LPT5"), TEXT("LPT6"), TEXT("LPT7"), TEXT("LPT8"), TEXT("LPT9")
};

/*-----------------------------------------------------------------------------
	FFileHelper
-----------------------------------------------------------------------------*/

/**
 * Load a binary file to a dynamic array with two uninitialized bytes at end as padding.
 */
bool FFileHelper::LoadFileToArray( TArray<uint8>& Result, const TCHAR* Filename, uint32 Flags )
{
	FScopedLoadingState ScopedLoadingState(Filename);

	FArchive* Reader = IFileManager::Get().CreateFileReader( Filename, Flags );
	if( !Reader )
	{
		if (!(Flags & FILEREAD_Silent))
		{
			UE_LOGF(LogStreaming,Warning,"Failed to read file '%ls' error.",Filename);
		}
		return false;
	}
	int64 TotalSize64 = Reader->TotalSize();
	if ( TotalSize64+2 > MAX_int32 )
	{
		if (!(Flags & FILEREAD_Silent))
		{
			UE_LOGF(LogStreaming,Error,"File '%ls' is too large for 32-bit reader (%lld), use TArray64.",Filename,TotalSize64);
		}
		return false;
	}
	int32 TotalSize = (int32)TotalSize64;
	// Allocate slightly larger than file size to avoid re-allocation when caller null terminates file buffer
	Result.Reset( TotalSize + 2 );
	Result.AddUninitialized( TotalSize );
	Reader->Serialize(Result.GetData(), Result.Num());
	bool bSuccess = Reader->Close();
	delete Reader;
	return bSuccess;
}

/**
 * Load a binary file to a dynamic array with two uninitialized bytes at end as padding.
 * TArray64 version.
 */
bool FFileHelper::LoadFileToArray(TArray64<uint8>& Result, const TCHAR* Filename, uint32 Flags)
{
	FScopedLoadingState ScopedLoadingState(Filename);

	FArchive* Reader = IFileManager::Get().CreateFileReader(Filename, Flags);
	if (!Reader)
	{
		if (!(Flags & FILEREAD_Silent))
		{
			UE_LOGF(LogStreaming, Warning, "Failed to read file '%ls' error.", Filename);
		}
		return false;
	}
	int64 TotalSize = Reader->TotalSize();
	// Allocate slightly larger than file size to avoid re-allocation when caller null terminates file buffer
	Result.Reset(TotalSize + 2);
	Result.AddUninitialized(TotalSize);
	Reader->Serialize(Result.GetData(), Result.Num());
	bool bSuccess = Reader->Close();
	delete Reader;
	return bSuccess;
}

bool FFileHelper::LoadFileInBlocks(FStringView Filename, TFunctionRef<void(FMemoryView)> BlockVisitor,
	int64 Offset, int64 Size, uint32 Flags, int64 BlockSize)
{
	TUniquePtr<FArchive> FileReader(IFileManager::Get().CreateFileReader(*WriteToString<256>(Filename), Flags));
	if (!FileReader)
	{
		return false;
	}

	const int64 DiskSize = FileReader->TotalSize();;
	Offset = FMath::Clamp(Offset, (int64)0, DiskSize);
	int64 RemainingLength = DiskSize - Offset;
	RemainingLength = Size < 0 ? RemainingLength : FMath::Min(Size, RemainingLength);

	if (RemainingLength > 0)
	{
		if (Offset > 0)
		{
			FileReader->Seek(Offset);
		}

		constexpr int64 DefaultMaxBufferSize = 1024 * 1024;
		const int64 MaxBufferSize = BlockSize <= 0 ? DefaultMaxBufferSize : BlockSize;
		int64 BufferSize = FMath::Min(MaxBufferSize, RemainingLength);
		TUniquePtr<uint8[]> Buffer(new uint8[BufferSize]);
		while (RemainingLength > 0)
		{
			int64 ReadLength = FMath::Min(RemainingLength, BufferSize);
			FileReader->Serialize(Buffer.Get(), ReadLength);
			RemainingLength -= ReadLength;
			BlockVisitor(FMemoryView(Buffer.Get(), ReadLength));
		}
		if (FileReader->IsError())
		{
			return false;
		}
	}

	return true;
}

/**
 * Converts an arbitrary text buffer to an FString.
 * Supports all combination of ANSI/Unicode files and platforms.
 */
void FFileHelper::BufferToString( FString& Result, const uint8* Buffer, int32 Size )
{
	TArray<TCHAR, FString::AllocatorType>& ResultArray = Result.GetCharArray();
	ResultArray.Empty();

	if( Size >= 2 && !( Size & 1 ) && Buffer[0] == 0xff && Buffer[1] == 0xfe )
	{
		// Unicode Intel byte order. Less 1 for the FFFE header, additional 1 for null terminator.
		ResultArray.AddUninitialized( Size / 2 );
		for( int32 i = 0; i < ( Size / 2 ) - 1; i++ )
		{
			ResultArray[ i ] = CharCast<TCHAR>( (UCS2CHAR)(( uint16 )Buffer[i * 2 + 2] + ( uint16 )Buffer[i * 2 + 3] * 256) );
		}
	}
	else if( Size >= 2 && !( Size & 1 ) && Buffer[0] == 0xfe && Buffer[1] == 0xff )
	{
		// Unicode non-Intel byte order. Less 1 for the FFFE header, additional 1 for null terminator.
		ResultArray.AddUninitialized( Size / 2 );
		for( int32 i = 0; i < ( Size / 2 ) - 1; i++ )
		{
			ResultArray[ i ] = CharCast<TCHAR>( (UCS2CHAR)(( uint16 )Buffer[i * 2 + 3] + ( uint16 )Buffer[i * 2 + 2] * 256) );
		}
	}
	else
	{
		if ( Size >= 3 && Buffer[0] == 0xef && Buffer[1] == 0xbb && Buffer[2] == 0xbf )
		{
			// Skip over UTF-8 BOM if there is one
			Buffer += 3;
			Size   -= 3;
		}

		int32 Length = FUTF8ToTCHAR_Convert::ConvertedLength(reinterpret_cast<const ANSICHAR*>(Buffer), Size);
		ResultArray.AddUninitialized(Length + 1); // +1 for the null terminator
		FUTF8ToTCHAR_Convert::Convert(ResultArray.GetData(), ResultArray.Num(), reinterpret_cast<const ANSICHAR*>(Buffer), Size);
		ResultArray[Length] = TEXT('\0');
	}

	if (ResultArray.Num() == 1)
	{
		// If it's only a zero terminator then make the result actually empty
		ResultArray.Empty();
	}
	else
	{
		// Else ensure null terminator is present
		ResultArray.Last() = TCHAR('\0');
	}
}

bool FFileHelper::LoadFileToString(FString& Result, FArchive& Reader, EHashOptions VerifyFlags /*= EHashOptions::None*/)
{
	FScopedLoadingState ScopedLoadingState(*Reader.GetArchiveName());

	int64 Size = Reader.TotalSize();
	if (!Size)
	{
		Result.Empty();
		return true;
	}

	if (Reader.Tell() != 0)
	{
		UE_LOGF(LogStreaming, Warning, "Archive '%ls' has already been read from.", *Reader.GetArchiveName());
		return false;
	}

	uint8* Ch = (uint8*)FMemory::Malloc(Size);
	Reader.Serialize(Ch, Size);
	bool bSuccess = !Reader.IsError();

	BufferToString(Result, Ch, (int32)Size);

	// Handle SHA verification of the file.
	if (EnumHasAnyFlags(VerifyFlags, EHashOptions::EnableVerify) && (EnumHasAnyFlags(VerifyFlags, EHashOptions::ErrorMissingHash) || FSHA1::GetFileSHAHash(*Reader.GetArchiveName(), nullptr)))
	{
		// Kick off SHA verify task. This frees the buffer on close.
		FBufferReaderWithSHA Ar(Ch, Size, true, *Reader.GetArchiveName(), false, true);
	}
	else
	{
		// Free manually, since the SHA task is not being run.
		FMemory::Free(Ch);
	}

	return bSuccess;
}

/**
 * Load a text file to an FString.
 * Supports all combination of ANSI/Unicode files and platforms.
 * @param Result string representation of the loaded file
 * @param Filename name of the file to load
 * @param VerifyFlags flags controlling the hash verification behavior ( see EHashOptions )
 */
bool FFileHelper::LoadFileToString(FString& Result, const TCHAR* Filename, EHashOptions VerifyFlags, uint32 ReadFlags)
{
	bool bSuccess = false;

	Result = AutoRTFM::Open([Filename, VerifyFlags, ReadFlags, &bSuccess]() -> FString
	{
		FString FileData;

		if (TUniquePtr<FArchive> Reader(IFileManager::Get().CreateFileReader(Filename, ReadFlags)); Reader)
		{
			bSuccess = LoadFileToString(FileData, *Reader, VerifyFlags);
		}

		return FileData;
	});

	return bSuccess;
}

bool FFileHelper::LoadFileToString(FString& Result, IPlatformFile* PlatformFile, const TCHAR* Filename, EHashOptions VerifyFlags /*= EHashOptions::None*/, uint32 ReadFlags /*= 0*/)
{
	bool bSuccess = false;

	if (PlatformFile)
	{
		Result = AutoRTFM::Open([PlatformFile, Filename, VerifyFlags, ReadFlags, &bSuccess]() -> FString
		{
			FString FileData;

			if (IFileHandle* File = PlatformFile->OpenRead(Filename))
			{
				FArchiveFileReaderGeneric Reader{File, Filename, File->Size()};
				bSuccess = LoadFileToString(FileData, Reader, VerifyFlags);
			}
			else
			{
				if (ReadFlags & FILEREAD_NoFail)
				{
					UE_LOGF(LogStreaming, Fatal, "Failed to read file: %ls", Filename);
				}

				if (!(ReadFlags & FILEREAD_Silent))
				{
					UE_LOGF(LogStreaming, Warning, "Failed to read file '%ls' error.", Filename);
				}
			}

			return FileData;
		});

	}

	return bSuccess;
}

namespace UE::Core::Private
{
static UE::Tasks::FPipe CloseHandlePipe(TEXT("CloseFileHandlesAndRequests"));
}

UE::Tasks::TTask<TValueOrError<FString, FFileSystemError>> FFileHelper::LoadFileToStringAsync(const TCHAR* Filename, FLoadFileAsyncOptions Options)
{
	IPlatformFile& PlatformFile = Options.OverridePlatformFile ? *Options.OverridePlatformFile : FPlatformFileManager::Get().GetPlatformFile();
	FFileOpenAsyncResult Result = PlatformFile.OpenAsyncRead(Filename, IPlatformFile::EOpenReadFlags::None);
	if (Result.HasError())
	{
		return UE::Tasks::MakeCompletedTask<TValueOrError<FString, FFileSystemError>>(MakeError(Result.StealError()));
	}

	// Requests must be deleted before the handle, and cannot be deleted in their own callbacks
	struct FState
	{
		TUniquePtr<IAsyncReadFileHandle> Handle;
		// The size and data requests must be deleted after their callback has finished executing
		// The callbacks may nest in the callstack if ReadRequest returns instantly with cached data so the SizeRequest cannot be deleted in the read callback.
		TUniquePtr<IAsyncReadRequest> SizeRequest;
		TUniquePtr<IAsyncReadRequest> DataRequest;
		UE::Tasks::FTaskEvent DataEvent{TEXT("LoadFileToStringAsync_DataEvent")};
		int64 Size = 0;
		TUniquePtr<uint8[]> Data;
		FString HashFilename;
		EAsyncIOPriorityAndFlags PriorityAndFlags;
		EHashOptions VerifyFlags;
	};
	TSharedPtr<FState> State = MakeShared<FState>(FState
	{
		.Handle = Result.StealValue(),
		.HashFilename = EnumHasAnyFlags(Options.VerifyFlags, EHashOptions::EnableVerify) ? FString(Filename) : FString(),
		.PriorityAndFlags = Options.PriorityAndFlags,
		.VerifyFlags = Options.VerifyFlags
	});

	// Note that callbacks are passed by ptr because they are optional.
	// The API doesn't require the supplied pointer to live past the function call, just that the copy remain safely callable.
	TFunction<void(bool, IAsyncReadRequest*)> SizeCallback = [State](bool bCancelled, IAsyncReadRequest* Request) mutable
	{
		State->SizeRequest.Reset(Request);
		State->Size = Request->GetSizeResults();
		if (State->Size <= 0 || State->Size > (int64)MAX_int32) // Result is -1 on cancellation or error
		{
			// Trigger final task without reading
			State->DataEvent.Trigger();
			return;
		}
		TFunction<void(bool, IAsyncReadRequest*)> ReadCallback = [State](bool bCancelled, IAsyncReadRequest* Request) mutable
		{
			State->DataRequest.Reset(Request);
			State->DataEvent.Trigger();
		};
		State->Data = MakeUnique<uint8[]>(State->Size);
		State->Handle->ReadRequest(0, State->Size, State->PriorityAndFlags, &ReadCallback, State->Data.Get());
	};
	State->Handle->SizeRequest(&SizeCallback);
	
	if (EnumHasAnyFlags(Options.VerifyFlags, EHashOptions::EnableVerify))
	{
		State->HashFilename = Filename;
	}

	return UE::Tasks::Launch(TEXT("LoadFileToStringAsync"), [State]() -> TValueOrError<FString, FFileSystemError>
		{
			// The start of this task can race with the IAsyncReadRequest code which calls the callback for us so that the completion flag is not properly set yet
			// Possibly overkill, but if the handles can't be released immediately, let a task do it
			// Put these tasks into a pipe so that an uncooperative API can only block a single worker
			// The file handle also shouldn't be closed until the requests are deleted
			UE::Core::Private::CloseHandlePipe.Launch(TEXT("DeleteReadRequests"), 
				[SizeRequest=MoveTemp(State->SizeRequest),DataRequest=MoveTemp(State->DataRequest),Handle=MoveTemp(State->Handle)]() mutable
				{
					SizeRequest->EnsureCompletion();
					SizeRequest.Reset();
					if (DataRequest)
					{
						DataRequest->EnsureCompletion();
						DataRequest.Reset();
					}
					Handle.Reset();
				});

			if (State->Size < 0)
			{
				// Async read handle doesn't provide detailed errors at this point
				return MakeError(FFileSystemError(TEXT("Unknown error")));
			}
			else if (State->Size == 0)
			{
				return MakeValue(FString());
			}
			else if (State->Size > (int64)MAX_int32)
			{
				return MakeError(FFileSystemError(FString::Printf(TEXT("File too large (%lld) for FString"), State->Size)));
			}

			FString String;
			if (State->Data.IsValid())
			{
				FFileHelper::BufferToString(String, State->Data.Get(), (int32)State->Size);
			}

			// Handle SHA verification of the file.
			if (EnumHasAnyFlags(State->VerifyFlags, EHashOptions::EnableVerify) 
			&& (EnumHasAnyFlags(State->VerifyFlags, EHashOptions::ErrorMissingHash) || FSHA1::GetFileSHAHash(*State->HashFilename, nullptr)))
			{
				// Buffer will be freed after async verification is finished
				FBufferReaderWithSHA Ar(
					(void*)State->Data.Release(),
					State->Size, 
					/* free buffer on close */ true, 
					*State->HashFilename, 
					/* bIsPersistent */ false, 
					/* bIsUnfoundHashAnError */ true);
			}

			return MakeValue(MoveTemp(String));
		},
		UE::Tasks::Prerequisites(State->DataEvent)
	);
}

bool FFileHelper::LoadFileToStringArray( TArray<FString>& Result, const TCHAR* Filename )
{
	return LoadFileToStringArrayWithPredicate(Result, Filename, [](const FString&) { return true;  });
}

bool FFileHelper::LoadFileToStringArrayWithPredicate(TArray<FString>& Result, const TCHAR* Filename, TFunctionRef<bool(const FString&)> Predicate)
{
	Result.Empty();

	TArray64<uint8> RawBuffer;
	// can be silent here, since returning false is enough
	if (!LoadFileToArray(RawBuffer, Filename, FILEREAD_Silent))
	{
		return false;
	}

	// we only support the 64-bit enabled "per-line conversion" functionality for UTF-8/ANSI strings, because the \r checks against a byte may fail
	// so we have to use the old "full string conversion" method, which doesn't work with 64-bits worth of data
	if (RawBuffer.Num() >= 2 && !(RawBuffer.Num() & 1) && 
		((RawBuffer[0] == 0xFF && RawBuffer[1] == 0xFE) || (RawBuffer[0] == 0xFE && RawBuffer[1] == 0xFF)))
	{
		// make sure we can use 32-bit algorithm
		if (RawBuffer.Num() > MAX_int32)
		{
			UE_LOGF(LogStreaming, Error, "A widechar format file used in LoadFileToStringArray[WithPredicate], but it's too large to be processed. File: %ls", Filename);
			return false;
		}

		FString Buffer;
		BufferToString(Buffer, RawBuffer.GetData(), (int32)RawBuffer.Num());

		for (const TCHAR* Pos = *Buffer; *Pos != 0; )
		{
			const TCHAR* LineStart = Pos;
			while (*Pos != 0 && *Pos != '\r' && *Pos != '\n')
			{
				Pos++;
			}

			FString Line = FString::ConstructFromPtrSize(LineStart, UE_PTRDIFF_TO_INT32(Pos - LineStart));
			if (Invoke(Predicate, Line))
			{
				Result.Add(MoveTemp(Line));
			}

			if (*Pos == '\r')
			{
				Pos++;
			}
			if (*Pos == '\n')
			{
				Pos++;
			}
		}

		return true;
	}


	int64 Length = RawBuffer.Num();
	for (const uint8* Pos = (uint8*)RawBuffer.GetData(); Length > 0; )
	{
		const uint8* LineStart = Pos;
		while (Length > 0 && *Pos != '\r' && *Pos != '\n')
		{
			Pos++;
			Length--;
		}

		if (Pos - LineStart > MAX_int32)
		{
			UE_LOGF(LogStreaming, Error, "Single line too long found in LoadFileToStringArrayWithPredicate, File: %ls", Filename);
			return false;
		}

		FString Line;
		BufferToString(Line, LineStart, UE_PTRDIFF_TO_INT32(Pos - LineStart));
		
		if (Invoke(Predicate, Line))
		{
			Result.Add(MoveTemp(Line));
		}

		if (*Pos == '\r')
		{
			Pos++;
			Length--;
		}
		if (*Pos == '\n')
		{
			Pos++;
			Length--;
		}
	}

	return true;
}

namespace UE::FileHelper::Private
{

enum class EEncoding
{
	Unknown,
	UTF8,
	UTF16BE,
	UTF16LE,
};

static EEncoding ParseEncoding(FMutableMemoryView& Buffer, const uint64 TotalSize)
{
	const uint64 Size = Buffer.GetSize();
	const uint8* const Bytes = static_cast<const uint8*>(Buffer.GetData());
	if (!(TotalSize & 1) && Size >= 2 && Bytes[0] == 0xff && Bytes[1] == 0xfe)
	{
		Buffer += 2;
		return EEncoding::UTF16LE;
	}
	if (!(TotalSize & 1) && Size >= 2 && Bytes[0] == 0xfe && Bytes[1] == 0xff)
	{
		Buffer += 2;
		return EEncoding::UTF16BE;
	}
	if (Size >= 3 && Bytes[0] == 0xef && Bytes[1] == 0xbb && Bytes[2] == 0xbf)
	{
		Buffer += 3;
	}
	return EEncoding::UTF8;
}

static FMutableMemoryView ParseLinesUTF8(FMutableMemoryView Buffer, TFunctionRef<void(FStringView Line)> Visitor, bool bLastBuffer)
{
	const uint8* Pos = static_cast<const uint8*>(Buffer.GetData());
	const uint8* const End = static_cast<const uint8*>(Buffer.GetDataEnd());
	while (Pos < End)
	{
		const uint8* const Line = Pos;

		while (Pos < End && *Pos != '\r' && *Pos != '\n')
		{
			++Pos;
		}

		if (!bLastBuffer && (Pos == End || (Pos + 1 == End && *Pos == '\r')))
		{
			return Buffer.Right(static_cast<uint64>(End - Line));
		}

		Visitor(FUTF8ToTCHAR(reinterpret_cast<const UTF8CHAR*>(Line), UE_PTRDIFF_TO_INT32(Pos - Line)));

		if (Pos < End && *Pos == '\r')
		{
			++Pos;
		}
		if (Pos < End && *Pos == '\n')
		{
			++Pos;
		}
	}
	return Buffer.Right(0);
}

static FMutableMemoryView ParseLinesUTF16BE(FMutableMemoryView Buffer, TFunctionRef<void(FStringView Line)> Visitor, bool bLastBuffer)
{
	uint16* Pos = static_cast<uint16*>(Buffer.GetData());
	uint16* const End = static_cast<uint16*>(Buffer.GetDataEnd());
	while (Pos < End)
	{
		uint16* const Line = Pos;

		while (Pos < End)
		{
			if (const uint16 CodeUnit = NETWORK_ORDER16(*Pos); CodeUnit == '\r' || CodeUnit == '\n')
			{
				break;
			}
			++Pos;
		}

		if (!bLastBuffer && (Pos == End || (Pos + 1 == End && NETWORK_ORDER16(*Pos) == '\r')))
		{
			return Buffer.Right(static_cast<uint64>(End - Line) * sizeof(uint16));
		}

		if constexpr (PLATFORM_LITTLE_ENDIAN)
		{
			for (uint16* SwapPos = Line; SwapPos < Pos; ++SwapPos)
			{
				*SwapPos = NETWORK_ORDER16(*SwapPos);
			}
		}

		Visitor(FUTF16ToTCHAR(reinterpret_cast<const UTF16CHAR*>(Line), UE_PTRDIFF_TO_INT32(Pos - Line)));

		if (Pos < End && NETWORK_ORDER16(*Pos) == '\r')
		{
			++Pos;
		}
		if (Pos < End && NETWORK_ORDER16(*Pos) == '\n')
		{
			++Pos;
		}
	}
	return Buffer.Right(0);
}

static FMutableMemoryView ParseLinesUTF16LE(FMutableMemoryView Buffer, TFunctionRef<void(FStringView Line)> Visitor, bool bLastBuffer)
{
	uint16* Pos = static_cast<uint16*>(Buffer.GetData());
	uint16* const End = static_cast<uint16*>(Buffer.GetDataEnd());
	while (Pos < End)
	{
		uint16* const Line = Pos;

		while (Pos < End)
		{
			if (const uint16 CodeUnit = INTEL_ORDER16(*Pos); CodeUnit == '\r' || CodeUnit == '\n')
			{
				break;
			}
			++Pos;
		}

		if (!bLastBuffer && (Pos == End || (Pos + 1 == End && INTEL_ORDER16(*Pos) == '\r')))
		{
			return Buffer.Right(static_cast<uint64>(End - Line) * sizeof(uint16));
		}

		if constexpr (!PLATFORM_LITTLE_ENDIAN)
		{
			for (uint16* SwapPos = Line; SwapPos < Pos; ++SwapPos)
			{
				*SwapPos = INTEL_ORDER16(*SwapPos);
			}
		}

		Visitor(FUTF16ToTCHAR(reinterpret_cast<const UTF16CHAR*>(Line), UE_PTRDIFF_TO_INT32(Pos - Line)));

		if (Pos < End && INTEL_ORDER16(*Pos) == '\r')
		{
			++Pos;
		}
		if (Pos < End && INTEL_ORDER16(*Pos) == '\n')
		{
			++Pos;
		}
	}
	return Buffer.Right(0);
}

} // UE::FileHelper::Private

bool FFileHelper::LoadFileToStringWithLineVisitor(const TCHAR* Filename, TFunctionRef<void(FStringView Line)> Visitor)
{
	using namespace UE::FileHelper::Private;

	FScopedLoadingState ScopedLoadingState(Filename);
	TUniquePtr<FArchive> Ar(IFileManager::Get().CreateFileReader(Filename, FILEREAD_Silent));
	if (!Ar)
	{
		return false;
	}

	const int64 TotalSize = Ar->TotalSize();
	FUniqueBuffer Buffer = FUniqueBuffer::Alloc(FMath::Min<int64>(TotalSize, 1024 * 1024));

	EEncoding Encoding = EEncoding::Unknown;
	FMutableMemoryView BufferTail = Buffer;
	for (int64 RemainingSize = TotalSize; RemainingSize > 0;)
	{
		const FMutableMemoryView SerializeBuffer = BufferTail.Left(RemainingSize);
		Ar->Serialize(SerializeBuffer.GetData(), static_cast<int64>(SerializeBuffer.GetSize()));
		RemainingSize -= SerializeBuffer.GetSize();

		FMutableMemoryView ParseBuffer = Buffer.GetView().LeftChop(BufferTail.GetSize() - SerializeBuffer.GetSize());

		if (Encoding == EEncoding::Unknown)
		{
			Encoding = ParseEncoding(ParseBuffer, TotalSize);
		}

		switch (Encoding)
		{
		case EEncoding::UTF8:
			ParseBuffer = ParseLinesUTF8(ParseBuffer, Visitor, RemainingSize == 0);
			break;
		case EEncoding::UTF16BE:
			ParseBuffer = ParseLinesUTF16BE(ParseBuffer, Visitor, RemainingSize == 0);
			break;
		case EEncoding::UTF16LE:
			ParseBuffer = ParseLinesUTF16LE(ParseBuffer, Visitor, RemainingSize == 0);
			break;
		default:
			checkNoEntry();
			return false;
		}

		if (Buffer.GetSize() == ParseBuffer.GetSize())
		{
			// No line endings were found. Double the buffer size and try again.
			FUniqueBuffer NewBuffer = FUniqueBuffer::Alloc(2 * Buffer.GetSize());
			BufferTail = NewBuffer.GetView().CopyFrom(ParseBuffer);
			Buffer = MoveTemp(NewBuffer);
		}
		else
		{
			// At least one line ending was found. Move any partial line to the front of the buffer and continue.
			FMemory::Memmove(Buffer.GetData(), ParseBuffer.GetData(), ParseBuffer.GetSize());
			BufferTail = Buffer.GetView() + ParseBuffer.GetSize();
		}
	}

	return Ar->Close();
}

/**
 * Save a binary array to a file.
 */
bool FFileHelper::SaveArrayToFile(TArrayView64<const uint8> Array, const TCHAR* Filename, IFileManager* FileManager /*= &IFileManager::Get()*/, uint32 WriteFlags)
{
	TUniquePtr<FArchive> Ar = TUniquePtr<FArchive>( FileManager->CreateFileWriter( Filename, WriteFlags ) );
	if( !Ar )
	{
		return false;
	}
	Ar->Serialize(const_cast<uint8*>(Array.GetData()), Array.Num());

	// Always explicitly close to catch errors from flush/close
	Ar->Close();

	return !Ar->IsError() && !Ar->IsCriticalError();
}

/**
 * Save a binary array to a file.
 */
bool FFileHelper::SaveArrayToFile(const TArray64<uint8>& Array, const TCHAR* Filename, IFileManager* FileManager /*= &IFileManager::Get()*/, uint32 WriteFlags /*= 0*/)
{
	TUniquePtr<FArchive> Ar = TUniquePtr<FArchive>(FileManager->CreateFileWriter(Filename, WriteFlags));
	if (!Ar)
	{
		return false;
	}
	Ar->Serialize(const_cast<uint8*>(Array.GetData()), Array.Num());

	// Always explicitly close to catch errors from flush/close
	Ar->Close();

	return !Ar->IsError() && !Ar->IsCriticalError();
}

/**
 * Write the FString to a file.
 * Supports all combination of ANSI/Unicode files and platforms.
 */
bool FFileHelper::SaveStringToFile( FStringView String, const TCHAR* Filename,  EEncodingOptions EncodingOptions, IFileManager* FileManager /*= &IFileManager::Get()*/, uint32 WriteFlags )
{
	// max size of the string is a UCS2CHAR for each character and some UNICODE magic 
	TUniquePtr<FArchive> Ar = TUniquePtr<FArchive>( FileManager->CreateFileWriter( Filename, WriteFlags ) );
	if (!Ar)
	{
		//UE_LOGF(LogStreaming, Warning, "SaveStringToFile: Failed to open writer. File:%ls", Filename);
		return false;
	}

	if (!String.IsEmpty())
	{
		bool SaveAsUnicode = EncodingOptions == EEncodingOptions::ForceUnicode || (EncodingOptions == EEncodingOptions::AutoDetect && !FCString::IsPureAnsi(String.GetData(), String.Len()));
		if (EncodingOptions == EEncodingOptions::ForceUTF8)
		{
			UTF8CHAR UTF8BOM[] = { (UTF8CHAR)0xEF, (UTF8CHAR)0xBB, (UTF8CHAR)0xBF };
			Ar->Serialize(&UTF8BOM, UE_ARRAY_COUNT(UTF8BOM) * sizeof(UTF8CHAR));

			FTCHARToUTF8 UTF8String(String.GetData(), String.Len());
			Ar->Serialize((UTF8CHAR*)UTF8String.Get(), UTF8String.Length() * sizeof(UTF8CHAR));
		}
		else if (EncodingOptions == EEncodingOptions::ForceUTF8WithoutBOM)
		{
			FTCHARToUTF8 UTF8String(String.GetData(), String.Len());
			Ar->Serialize((UTF8CHAR*)UTF8String.Get(), UTF8String.Length() * sizeof(UTF8CHAR));
		}
		else if (SaveAsUnicode)
		{
			UTF16CHAR BOM = UNICODE_BOM;
			Ar->Serialize(&BOM, sizeof(UTF16CHAR));

			// Note: This is a no-op on platforms that are using a 16-bit TCHAR
			FTCHARToUTF16 UTF16String(String.GetData(), String.Len());
			Ar->Serialize((UTF16CHAR*)UTF16String.Get(), UTF16String.Length() * sizeof(UTF16CHAR));
		}
		else
		{
			auto Src = StringCast<ANSICHAR>(String.GetData(), String.Len());
			Ar->Serialize((ANSICHAR*)Src.Get(), Src.Length() * sizeof(ANSICHAR));
		}
	}

	// Always explicitly close to catch errors from flush/close
	Ar->Close();

	if (Ar->IsError())
	{
		UE_LOGF(LogStreaming, Warning, "SaveStringToFile: Ar->IsError() == true. File:%ls", Filename);
	}
	if (Ar->IsCriticalError())
	{
		UE_LOGF(LogStreaming, Warning, "SaveStringToFile: Ar->IsCriticalError() == true. File:%ls", Filename);
	}
	return !Ar->IsError() && !Ar->IsCriticalError();
}

bool FFileHelper::SaveStringArrayToFile( const TArray<FString>& Lines, const TCHAR* Filename, EEncodingOptions EncodingOptions, IFileManager* FileManager, uint32 WriteFlags )
{
	int32 Length = 10;
	for(const FString& Line : Lines)
	{
		Length += Line.Len() + UE_ARRAY_COUNT(LINE_TERMINATOR);
	}
	
	FString CombinedString;
	CombinedString.Reserve(Length);

	for(const FString& Line : Lines)
	{
		CombinedString += Line;
		CombinedString += LINE_TERMINATOR;
	}

	return SaveStringToFile(CombinedString, Filename, EncodingOptions, FileManager, WriteFlags);
}

/**
 * Generates the next unique bitmap filename with a specified extension
 * 
 * @param Pattern		Filename with path, but without extension.
 * @oaran Extension		File extension to be appended
 * @param OutFilename	Reference to an FString where the newly generated filename will be placed
 * @param FileManager	Reference to a IFileManager (or the global instance by default)
 *
 * @return true if success
 */
bool FFileHelper::GenerateNextBitmapFilename( const FString& Pattern, const FString& Extension, FString& OutFilename, IFileManager* FileManager /*= &IFileManager::Get()*/ )
{
	FString File;
	OutFilename = "";
	bool bSuccess = false;

	//
	// As an optimization for sequential screenshots using the same pattern, we track the last index used and check if that exists 
	// for the provided pattern. If it does we start checking from that index
	// 
	// If a file with the last used index does not exist we it's a different pattern so start at 0 to find the next free name.

	static int32 LastScreenShotIndex = 0;
	int32 SearchIndex = 0;

	File = FString::Printf(TEXT("%s%05i.%s"), *Pattern, LastScreenShotIndex, *Extension);

	if (FileManager->FileExists(*File))
	{
		SearchIndex = LastScreenShotIndex+1;
	}
	
	for( int32 TestBitmapIndex = SearchIndex; TestBitmapIndex < 100000; ++TestBitmapIndex )
	{
		File = FString::Printf(TEXT("%s%05i.%s"), *Pattern, TestBitmapIndex, *Extension);
		if( FileManager->FileExists(*File) == false)
		{
			LastScreenShotIndex = TestBitmapIndex;
			OutFilename = File;
			bSuccess = true;
			break;
		}
	}

	return bSuccess;
}

/**
 * Generates the next unique bitmap filename with a specified extension
 *
 * @param Pattern		Filename with path, but without extension.
 * @oaran Extension		File extension to be appended
 * @param OutFilename	Reference to an FString where the newly generated filename will be placed
 *
 * @return true if success
 */
void FFileHelper::GenerateDateTimeBasedBitmapFilename(const FString& Pattern, const FString& Extension, FString& OutFilename)
{
	// Use current date & time to obtain more organized screenshot libraries
	// There is no need to check for file duplicate, as two certian moments, can't occure twice in the world!
	
	OutFilename = "";

	static int32 LastScreenShotIndex = 0;
	int32 SearchIndex = 0;

	OutFilename = FString::Printf(TEXT("%s_%s.%s"), *Pattern, *FDateTime::Now().ToString(), *Extension);
}

bool FFileHelper::CreateBitmap( const TCHAR* Pattern, int32 SourceWidth, int32 SourceHeight, const FColor* Data, FIntRect* SubRectangle, IFileManager* FileManager, FString* OutFilename, bool bInWriteAlpha, EColorChannel ColorChannel)
{
	FIntRect Src(0, 0, SourceWidth, SourceHeight);
	if (SubRectangle == NULL || SubRectangle->Area() == 0)
	{
		SubRectangle = &Src;
	}

	FString File;
	// if the Pattern already has a .bmp extension, then use that the file to write to
	if (FPaths::GetExtension(Pattern) == TEXT("bmp"))
	{
		File = Pattern;
	}
	else
	{
		if (GenerateNextBitmapFilename(Pattern, TEXT("bmp"), File, FileManager))
		{
			if ( OutFilename )
			{
				*OutFilename = File;
			}
		}
		else
		{
			return false;
		}
	}

	FArchive* Ar = FileManager->CreateFileWriter( *File );
	if( Ar )
	{
		// Types.
		#if PLATFORM_SUPPORTS_PRAGMA_PACK
			#pragma pack (push,1)
		#endif
		struct BITMAPFILEHEADER
		{
			uint16 bfType GCC_PACK(1);
			uint32 bfSize GCC_PACK(1);
			uint16 bfReserved1 GCC_PACK(1); 
			uint16 bfReserved2 GCC_PACK(1);
			uint32 bfOffBits GCC_PACK(1);
		} FH; 
		struct BITMAPINFOHEADER
		{
			uint32 biSize GCC_PACK(1); 
			int32  biWidth GCC_PACK(1);
			int32  biHeight GCC_PACK(1);
			uint16 biPlanes GCC_PACK(1);
			uint16 biBitCount GCC_PACK(1);
			uint32 biCompression GCC_PACK(1);
			uint32 biSizeImage GCC_PACK(1);
			int32  biXPelsPerMeter GCC_PACK(1); 
			int32  biYPelsPerMeter GCC_PACK(1);
			uint32 biClrUsed GCC_PACK(1);
			uint32 biClrImportant GCC_PACK(1); 
		} IH;
		struct BITMAPV4HEADER
		{
			uint32 bV4RedMask GCC_PACK(1);
			uint32 bV4GreenMask GCC_PACK(1);
			uint32 bV4BlueMask GCC_PACK(1);
			uint32 bV4AlphaMask GCC_PACK(1);
			uint32 bV4CSType GCC_PACK(1);
			uint32 bV4EndpointR[3] GCC_PACK(1);
			uint32 bV4EndpointG[3] GCC_PACK(1);
			uint32 bV4EndpointB[3] GCC_PACK(1);
			uint32 bV4GammaRed GCC_PACK(1);
			uint32 bV4GammaGreen GCC_PACK(1);
			uint32 bV4GammaBlue GCC_PACK(1);
		} IHV4;
		#if PLATFORM_SUPPORTS_PRAGMA_PACK
			#pragma pack (pop)
		#endif

		int32 Width = SubRectangle->Width();
		int32 Height = SubRectangle->Height();
		uint32 BytesPerPixel = bInWriteAlpha ? 4 : 3;
		uint32 BytesPerLine = Align(Width * BytesPerPixel, 4);

		uint32 InfoHeaderSize = sizeof(BITMAPINFOHEADER) + (bInWriteAlpha ? sizeof(BITMAPV4HEADER) : 0);

		// File header.
		FH.bfType       		= INTEL_ORDER16((uint16) ('B' + 256*'M'));
		FH.bfSize       		= INTEL_ORDER32((uint32) (sizeof(BITMAPFILEHEADER) + InfoHeaderSize + BytesPerLine * Height));
		FH.bfReserved1  		= INTEL_ORDER16((uint16) 0);
		FH.bfReserved2  		= INTEL_ORDER16((uint16) 0);
		FH.bfOffBits    		= INTEL_ORDER32((uint32) (sizeof(BITMAPFILEHEADER) + InfoHeaderSize));
		Ar->Serialize( &FH, sizeof(FH) );

		// Info header.
		IH.biSize               = INTEL_ORDER32((uint32) InfoHeaderSize);
		IH.biWidth              = INTEL_ORDER32((uint32) Width);
		IH.biHeight             = INTEL_ORDER32((uint32) Height);
		IH.biPlanes             = INTEL_ORDER16((uint16) 1);
		IH.biBitCount           = INTEL_ORDER16((uint16) BytesPerPixel * 8);
		if(bInWriteAlpha)
		{
			IH.biCompression    = INTEL_ORDER32((uint32) 3); //BI_BITFIELDS
		}
		else
		{
			IH.biCompression    = INTEL_ORDER32((uint32) 0); //BI_RGB
		}
		IH.biSizeImage          = INTEL_ORDER32((uint32) BytesPerLine * Height);
		IH.biXPelsPerMeter      = INTEL_ORDER32((uint32) 0);
		IH.biYPelsPerMeter      = INTEL_ORDER32((uint32) 0);
		IH.biClrUsed            = INTEL_ORDER32((uint32) 0);
		IH.biClrImportant       = INTEL_ORDER32((uint32) 0);
		Ar->Serialize( &IH, sizeof(IH) );

		// If we're writing alpha, we need to write the extra portion of the V4 header
		if (bInWriteAlpha)
		{
			IHV4.bV4RedMask     = INTEL_ORDER32((uint32) 0x00ff0000);
			IHV4.bV4GreenMask   = INTEL_ORDER32((uint32) 0x0000ff00);
			IHV4.bV4BlueMask    = INTEL_ORDER32((uint32) 0x000000ff);
			IHV4.bV4AlphaMask   = INTEL_ORDER32((uint32) 0xff000000);
			IHV4.bV4CSType      = INTEL_ORDER32((uint32) 'Win ');
			IHV4.bV4GammaRed    = INTEL_ORDER32((uint32) 0);
			IHV4.bV4GammaGreen  = INTEL_ORDER32((uint32) 0);
			IHV4.bV4GammaBlue   = INTEL_ORDER32((uint32) 0);
			Ar->Serialize( &IHV4, sizeof(IHV4) );
		}

		// Colors.
		// @todo fix me : calling Serialize per byte = insanely slow
		//	BmpImageWrapper now has a good writer, prefer that ; use FImageUtils::SaveImage
		for( int32 i = SubRectangle->Max.Y - 1; i >= SubRectangle->Min.Y; i-- )
		{
			for( int32 j = SubRectangle->Min.X; j < SubRectangle->Max.X; j++ )
			{
				if (ColorChannel == EColorChannel::All)
				{
					Ar->Serialize((void*)&Data[i * SourceWidth + j].B, 1);
					Ar->Serialize((void*)&Data[i * SourceWidth + j].G, 1);
					Ar->Serialize((void*)&Data[i * SourceWidth + j].R, 1);

					if (bInWriteAlpha)
					{
						Ar->Serialize((void*)&Data[i * SourceWidth + j].A, 1);
					}
				}
				else
				{
					const uint8 Max = 255;
					uint8 ChannelValue = 0;
					// When using Channel mask write the masked channel to all channels (except alpha).
					switch (ColorChannel)
					{
					case EColorChannel::B:
						ChannelValue = Data[i * SourceWidth + j].B;
						break;
					case EColorChannel::G:
						ChannelValue = Data[i * SourceWidth + j].G;
						break;
					case EColorChannel::R:
						ChannelValue = Data[i * SourceWidth + j].R;
						break;
					case EColorChannel::A:
						ChannelValue = Data[i * SourceWidth + j].A;
						break;
					}
										
					// replicate Channel in B, G, R
					Ar->Serialize((void*)&ChannelValue, 1);
					Ar->Serialize((void*)&ChannelValue, 1);
					Ar->Serialize((void*)&ChannelValue, 1);

					// if write alpha write max value in there (we don't want transparency)
					if (bInWriteAlpha)
					{
						Ar->Serialize((void*)&Max, 1);
					}
				}

				
			}

			// Pad each row's length to be a multiple of 4 bytes.

			for(uint32 PadIndex = Width * BytesPerPixel; PadIndex < BytesPerLine; PadIndex++)
			{
				uint8 B = 0;
				Ar->Serialize(&B, 1);
			}
		}

		// Success.
		delete Ar;
		if (!GIsEditor)
		{
			SendDataToPCViaUnrealConsole( TEXT("UE_PROFILER!BUGIT:"), File );
		}
	}
	else 
	{
		return false;
	}

	// Success.
	return true;
}

/**
 *	Load the given ANSI text file to an array of strings - one FString per line of the file.
 *	Intended for use in simple text parsing actions
 *
 *	@param	InFilename			The text file to read, full path
 *	@param	InFileManager		The filemanager to use - NULL will use &IFileManager::Get()
 *	@param	OutStrings			The array of FStrings to fill in
 *
 *	@return	bool				true if successful, false if not
 */
bool FFileHelper::LoadANSITextFileToStrings(const TCHAR* InFilename, IFileManager* InFileManager, TArray<FString>& OutStrings)
{
	FScopedLoadingState ScopedLoadingState(InFilename);

	IFileManager* FileManager = (InFileManager != NULL) ? InFileManager : &IFileManager::Get();
	// Read and parse the file, adding the pawns and their sounds to the list
	FArchive* TextFile = FileManager->CreateFileReader(InFilename, 0);
	if (TextFile != NULL)
	{
		// get the size of the file
		int32 Size = (int32)TextFile->TotalSize();
		// read the file
		TArray<uint8> Buffer;
		Buffer.Empty(Size + 1);
		Buffer.AddUninitialized(Size);
		TextFile->Serialize(Buffer.GetData(), Size);
		// zero terminate it
		Buffer.Add(0);
		// Release the file
		delete TextFile;

		// Now read it
		// init traveling pointer
		ANSICHAR* Ptr = (ANSICHAR*)Buffer.GetData();

		// iterate over the lines until complete
		bool bIsDone = false;
		while (!bIsDone)
		{
			// Store the location of the first character of this line
			ANSICHAR* Start = Ptr;

			// Advance the char pointer until we hit a newline character
			while (*Ptr && *Ptr!='\r' && *Ptr!='\n')
			{
				Ptr++;
			}

			// If this is the end of the file, we're done
			if (*Ptr == 0)
			{
				bIsDone = 1;
			}
			// Handle different line endings. If \r\n then NULL and advance 2, otherwise NULL and advance 1
			// This handles \r, \n, or \r\n
			else if ( *Ptr=='\r' && *(Ptr+1)=='\n' )
			{
				// This was \r\n. Terminate the current line, and advance the pointer forward 2 characters in the stream
				*Ptr++ = 0;
				*Ptr++ = 0;
			}
			else
			{
				// Terminate the current line, and advance the pointer to the next character in the stream
				*Ptr++ = 0;
			}

			FString CurrLine = ANSI_TO_TCHAR(Start);
			OutStrings.Add(CurrLine);
		}

		return true;
	}
	else
	{
		UE_LOGF(LogStreaming, Warning, "Failed to open ANSI TEXT file %ls", InFilename);
		return false;
	}
}

/**
* Checks to see if a filename is valid for saving.
* A filename must be under FPlatformMisc::GetMaxPathLength() to be saved
*
* @param Filename	Filename, with or without path information, to check.
* @param OutError	If an error occurs, this is the reason why
*/
bool FFileHelper::IsFilenameValidForSaving(const FString& Filename, FText& OutError)
{
	bool bFilenameIsValid = false;

	// Get the clean filename (filename with extension but without path )
	const FString BaseFilename = FPaths::GetBaseFilename(Filename);

	// Check length of the filename
	if (BaseFilename.Len() > 0)
	{
		if (BaseFilename.Len() <= FPlatformMisc::GetMaxPathLength())
		{
			bFilenameIsValid = true;

			/*
			// Check that the name isn't the name of a UClass
			for ( TObjectIterator<UClass> It; It; ++It )
			{
			UClass* Class = *It;
			if ( Class->GetName() == BaseFilename )
			{
			bFilenameIsValid = false;
			break;
			}
			}
			*/

			for (const TCHAR* InvalidFilename : InvalidFilenames)
			{
				if (BaseFilename.Equals(InvalidFilename, ESearchCase::IgnoreCase))
				{
					OutError = NSLOCTEXT("UnrealEd", "Error_InvalidFilename", "A file/folder may not match any of the following : \nCON, PRN, AUX, CLOCK$, NUL, NONE, \nCOM1, COM2, COM3, COM4, COM5, COM6, COM7, COM8, COM9, \nLPT1, LPT2, LPT3, LPT4, LPT5, LPT6, LPT7, LPT8, or LPT9.");
					return false;
				}
			}

			if (FName(*BaseFilename).IsNone())
			{
				OutError = FText::Format(NSLOCTEXT("UnrealEd", "Error_NoneFilename", "Filename '{0}' resolves to 'None' and cannot be used"), FText::FromString(BaseFilename));
				return false;
			}

			// Check for invalid characters in the filename
			if (bFilenameIsValid &&
				(BaseFilename.Contains(TEXT("."), ESearchCase::CaseSensitive, ESearchDir::FromEnd) ||
					BaseFilename.Contains(TEXT(":"), ESearchCase::CaseSensitive, ESearchDir::FromEnd)))
			{
				bFilenameIsValid = false;
			}

			if (!bFilenameIsValid)
			{
				OutError = FText::Format(NSLOCTEXT("UnrealEd", "Error_FilenameDisallowed", "Filename '{0}' is disallowed."), FText::FromString(BaseFilename));
			}
		}
		else
		{
			OutError = FText::Format(NSLOCTEXT("UnrealEd", "Error_FilenameIsTooLongForCooking", "Filename is too long ({0} characters); this may interfere with cooking for consoles. Unreal filenames should be no longer than {1} characters. Filename value: {2}"),
				FText::AsNumber(BaseFilename.Len()), FText::AsNumber(FPlatformMisc::GetMaxPathLength()), FText::FromString(BaseFilename));
		}
	}
	else
	{
		OutError = LOCTEXT("Error_FilenameIsTooShort", "Please provide a filename for the asset.");
	}

	return bFilenameIsValid;
}

/*-----------------------------------------------------------------------------
	FMaintenance
-----------------------------------------------------------------------------*/
void FMaintenance::DeleteOldLogs()
{
	SCOPED_BOOT_TIMING("FMaintenance::DeleteOldLogs");
	int32 PurgeLogsDays = -1; // -1 means don't delete old files
	int32 MaxLogFilesOnDisk = -1; // -1 means keep all files

	GConfig->GetInt(TEXT("LogFiles"), TEXT("PurgeLogsDays"), PurgeLogsDays, GEngineIni);
	GConfig->GetInt(TEXT("LogFiles"), TEXT("MaxLogFilesOnDisk"), MaxLogFilesOnDisk, GEngineIni);

	if (PurgeLogsDays >= 0 || MaxLogFilesOnDisk >= 0)
	{
		// get list of files in the log directory (grouped by log name)
		TMap<FString, TArray<FString>> LogToPaths;
		{
			TArray<FString> Files;
			IFileManager::Get().FindFiles(Files, *FString::Printf(TEXT("%s*.*"), *FPaths::ProjectLogDir()), true, false);

			for (FString& Filename : Files)
			{
				const int32 BackupPostfixIndex = Filename.Find(BACKUP_LOG_FILENAME_POSTFIX);

				if (BackupPostfixIndex >= 0)
				{
					const FString LogName = Filename.Left(BackupPostfixIndex);
					TArray<FString>& FilePaths = LogToPaths.FindOrAdd(LogName);
					FilePaths.Add(FPaths::ProjectLogDir() / Filename);
				}
			}
		}

		// delete old log files in each group
		double MaxFileAgeSeconds = 60.0 * 60.0 * 24.0 * double(PurgeLogsDays);

		struct FSortByDateNewestFirst
		{
			bool operator()(const FString& A, const FString& B) const
			{
				const FDateTime TimestampA = IFileManager::Get().GetTimeStamp(*A);
				const FDateTime TimestampB = IFileManager::Get().GetTimeStamp(*B);
				return TimestampB < TimestampA;
			}
		};

		for (TPair<FString, TArray<FString>>& Pair : LogToPaths)
		{
			TArray<FString>& FilePaths = Pair.Value;

			// sort the file paths by date
			FilePaths.Sort(FSortByDateNewestFirst());

			// delete files that are older than the desired number of days
			for (int32 PathIndex = FilePaths.Num() - 1; PathIndex >= 0; --PathIndex)
			{
				const FString& FilePath = FilePaths[PathIndex];

				if (IFileManager::Get().GetFileAgeSeconds(*FilePath) > MaxFileAgeSeconds)
				{
					UE_LOGF(LogStreaming, Log, "Deleting old log file %ls", *FilePath);
					IFileManager::Get().Delete(*FilePath);
					FilePaths.RemoveAt(PathIndex);
				}
			}

			// trim the number of files on disk if desired
			if (MaxLogFilesOnDisk >= 0 && FilePaths.Num() > MaxLogFilesOnDisk)
			{
				for (int32 PathIndex = FilePaths.Num() - 1; PathIndex >= 0 && FilePaths.Num() > MaxLogFilesOnDisk; --PathIndex)
				{
					if (FOutputDeviceFile::IsBackupCopy(*FilePaths[PathIndex]))
					{
						IFileManager::Get().Delete(*FilePaths[PathIndex]);
						FilePaths.RemoveAt(PathIndex);
					}
				}
			}
		}
	}

	// Remove all legacy crash contexts (regardless of age and purge settings, these are deprecated)
	TArray<FString> Directories;
	IFileManager::Get().FindFiles(Directories, *FString::Printf(TEXT("%s/UE4CC*"), *FPaths::ProjectLogDir()), false, true);
	IFileManager::Get().FindFiles(Directories, *FString::Printf(TEXT("%s/UECC*"), *FPaths::ProjectLogDir()), false, true);
	
	for (const FString& Dir : Directories)
	{
		const FString CrashConfigDirectory = FPaths::ProjectLogDir() / Dir;
		IFileManager::Get().DeleteDirectory(*CrashConfigDirectory, false, true);
	}
}

#undef LOCTEXT_NAMESPACE
