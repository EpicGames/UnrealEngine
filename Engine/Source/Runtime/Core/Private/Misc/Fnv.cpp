// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/Fnv.h"
/** Algorithm implementation from http://tools.ietf.org/html/draft-eastlake-fnv-16 */

// generate a 32-bit hash from data in memory of length bytes
uint32 FFnv::MemFnv32( const void* InData, int32 Length, uint32 InOffset )
{
    // constants from above reference
    static const uint32 Offset = 0x811c9dc5;
    static const uint32 Prime = 0x01000193;
    
    const uint8* __restrict Data = (uint8*)InData;
    
    uint32 Fnv = Offset+InOffset;   // this is not strictly correct as the offset should be prime and InOffset could be arbitrary
    for (; Length; --Length)
    {
        Fnv ^= *Data++;
        Fnv *= Prime;
    }
    
    return Fnv;
}

// generate a 64-bit hash from data in memory of length bytes
uint64 FFnv::MemFnv64( const void* InData, int32 Length, uint64 InOffset )
{
    // constants from above reference
    static const uint64 Offset = 0xcbf29ce484222325;
    static const uint64 Prime = 0x00000100000001b3;
    
    const uint8* __restrict Data = (uint8*)InData;
    
    uint64 Fnv = Offset+InOffset; // this is not strictly correct as the offset should be prime and InOffset could be arbitrary
    for (; Length; --Length)
    {
        Fnv ^= *Data++;
        Fnv *= Prime;
    }
    
    return Fnv;
}
