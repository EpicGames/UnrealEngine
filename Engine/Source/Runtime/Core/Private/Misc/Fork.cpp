// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/Fork.h"
#include "HAL/ThreadManager.h"
#include "HAL/RunnableThread.h"

#include "Misc/CommandLine.h"
#include "Misc/Parse.h"

namespace ForkInternal
{
	/** Set to true on a child process right after the fork point */
	static bool bIsForkedChildProcess = false;

	/** Will be set to true on a child process right before we spin up the ForkableThreads at the end of the frame where we forked. */
	static bool bIsForkedMultithreadInstance = false;

	/** The unique index of a forked child process */
	static uint16 ChildProcessIndex = 0;

	/**
	 * Are we doing a real fork and generating child processes on the fork process who received the -WaitAndFork commandline.
	 * Note: this will be true on the child processes too.
	 */
	bool IsRealForkRequested()
	{
		// We cache the value since only the master process might receive -WaitAndFork
		static const bool bRealForkRequested = FParse::Param(FCommandLine::IsInitialized() ? FCommandLine::Get() : TEXT(""), TEXT("WaitAndFork"));
		return bRealForkRequested;
	}

	/**
	* Fake Forking is when we run the Fork codepath without actually duplicating the process.
	* Useful to test the flow on platforms that do not support forking or to debug the process on fork-enabled platforms without having to attach to the new process.
	* Note: The master process is considered a child process after the Fork event and this function will continue returning true.
	*/
	bool IsFakeForking()
	{

#if !UE_SERVER
		// Only possible on server processes
		return false;
#else
		const bool bRealForkRequested = ForkInternal::IsRealForkRequested();
		if (bRealForkRequested)
		{
			return false;
		}

#if DEFAULT_SERVER_FAKE_FORKS
		// We always fake fork unless explicitely disabled
		static const bool bNoFakeForking = FParse::Param(FCommandLine::Get(), TEXT("NoFakeForking"));
		return !bNoFakeForking;
#else
		// Only run the fake forking path when requested
		static const bool bDoFakeForking = FParse::Param(FCommandLine::Get(), TEXT("FakeForking"));
		return bDoFakeForking;
#endif

#endif //#if !UE_SERVER
	}
}

bool FForkProcessHelper::IsForkedChildProcess()
{
	return ForkInternal::bIsForkedChildProcess;
}

void FForkProcessHelper::SetIsForkedChildProcess(uint16 ChildIndex)
{
	ForkInternal::bIsForkedChildProcess = true;
	ForkInternal::ChildProcessIndex = ChildIndex;
}

uint16 FForkProcessHelper::GetForkedChildProcessIndex()
{
	return ForkInternal::ChildProcessIndex;
}

bool FForkProcessHelper::IsForkRequested()
{
	return ForkInternal::IsRealForkRequested() || ForkInternal::IsFakeForking();
}


void FForkProcessHelper::OnForkingOccured()
{
	if (SupportsMultithreadingPostFork())
	{
		ensureMsgf(UE::Private::GMalloc->IsInternallyThreadSafe(), TEXT("The BaseAllocator %s is not threadsafe. Switch to a multithread allocator or ensure the FMallocThreadSafeProxy wraps it."), UE::Private::GMalloc->GetDescriptiveName());

		ForkInternal::bIsForkedMultithreadInstance = true;

		// Use a local list of forkable threads so we don't keep a lock on the global list during thread creation
		TArray<FRunnableThread*> ForkableThreads = FThreadManager::Get().GetForkableThreads();
		for (FRunnableThread* ForkableThread : ForkableThreads)
		{
			ForkableThread->OnPostFork();
		}
	}
}

bool FForkProcessHelper::IsForkedMultithreadInstance()
{
	return ForkInternal::bIsForkedMultithreadInstance;
}

bool FForkProcessHelper::SupportsMultithreadingPostFork()
{
	if (!FCommandLine::IsInitialized())
	{
		// Return the default setting if the cmdline isn't set yet.
		return DEFAULT_FORK_PROCESS_MULTITHREAD;
	}

#if DEFAULT_FORK_PROCESS_MULTITHREAD
	// Always multi thread unless manually turned off via command line
	static bool bSupportsMT = FParse::Param(FCommandLine::Get(), TEXT("DisablePostForkThreading")) == false;
	return bSupportsMT;
#else
	// Always single thread unless manually turned on via command line
	static bool bSupportsMT = FParse::Param(FCommandLine::Get(), TEXT("PostForkThreading")) == true;
	return bSupportsMT;
#endif
}

void FForkProcessHelper::LowLevelPreFork()
{
	UE::Private::GMalloc->OnPreFork();
}

void FForkProcessHelper::LowLevelPostForkParent()
{
	// Currently nothing to do here, just provided for completeness. 
}

void FForkProcessHelper::LowLevelPostForkChild(uint16 ChildIndex)
{
	FForkProcessHelper::SetIsForkedChildProcess(ChildIndex);
	UE::Private::GMalloc->OnPostFork();
}