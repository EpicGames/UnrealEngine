// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/FrameNumber.h"
#include "Serialization/Archive.h"



/* FFrameNumber interface
 *****************************************************************************/

bool FFrameNumber::Serialize(FArchive& Ar)
{
	Ar << Value;
	return true;
}

FArchive& operator<<(FArchive& Ar, FFrameNumber& FrameNumber)
{
	return Ar << FrameNumber.Value;
}
