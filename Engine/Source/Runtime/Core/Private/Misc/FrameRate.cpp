// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/FrameRate.h"

#include "Algo/BinarySearch.h"
#include "Algo/Reverse.h"
#include "Containers/Array.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Internationalization/Internationalization.h"
#include "Math/BasicMathExpressionEvaluator.h"
#include "Misc/ExpressionParser.h"
#include "Misc/Guid.h"
#include "Templates/UnrealTemplate.h"

#define LOCTEXT_NAMESPACE "FFrameRate"

namespace
{
	struct FSeconds         { static const TCHAR* const Moniker; };
	struct FFramesPerSecond { static const TCHAR* const Moniker; };
	struct FNonDropFrame    { static const TCHAR* const Moniker; };
	struct FDropFrame       { static const TCHAR* const Moniker; };

	const TCHAR* const FFramesPerSecond::Moniker = TEXT("fps");
	const TCHAR* const FSeconds::Moniker         = TEXT("s");
	const TCHAR* const FNonDropFrame::Moniker    = TEXT("ndf");
	const TCHAR* const FDropFrame::Moniker       = TEXT("df");

	const double NTSC_24_FPS = 23.976;
	const double NTSC_30_FPS = 29.97;
	const double NTSC_60_FPS = 59.94;

	const uint32 NTSC_24_Numerator = 24000u;
	const uint32 NTSC_30_Numerator = 30000u;
	const uint32 NTSC_60_Numerator = 60000u;
	const uint32 NTSC_Denominator  = 1001u;
}

DEFINE_EXPRESSION_NODE_TYPE(FFrameRate, 0x4EDAA92F, 0xB75E4B9E, 0xB7E0ABC2, 0x8D981FCB)
DEFINE_EXPRESSION_NODE_TYPE(FSeconds, 0x3DC5F60D, 0x934E4753, 0xA80CD6D0, 0xE9EB4640)
DEFINE_EXPRESSION_NODE_TYPE(FFramesPerSecond, 0x8423B4AE, 0x2FF64795, 0xA7EFFAC0, 0xC560531A)
DEFINE_EXPRESSION_NODE_TYPE(FNonDropFrame, 0xEE67D9FA, 0x435C4991, 0xDF530089, 0x8B3019D0)
DEFINE_EXPRESSION_NODE_TYPE(FDropFrame, 0x56AF8BD1, 0x431DF28A, 0x8ECC1EB7, 0xC00C5E5D)

const float FFrameTime::MaxSubframe = 0.99999994f;


/** A basic math expression evaluator */
class FFrameRateParser
{
public:
	/** Constructor that sets up the parser's lexer and compiler */
	FFrameRateParser()
	{
		using namespace ExpressionParser;

		TokenDefinitions.IgnoreWhitespace();
		TokenDefinitions.DefineToken(&ConsumeSymbol<FSeconds>);
		TokenDefinitions.DefineToken(&ConsumeSymbol<FFramesPerSecond>);
		TokenDefinitions.DefineToken(&ConsumeSymbol<FNonDropFrame>);
		TokenDefinitions.DefineToken(&ConsumeSymbol<FDropFrame>);
		TokenDefinitions.DefineToken(&ConsumeSymbol<FForwardSlash>);
		TokenDefinitions.DefineToken(&ConsumeLocalizedNumberWithAgnosticFallback);

		Grammar.DefineBinaryOperator<FForwardSlash>(1);
		Grammar.DefinePostUnaryOperator<FSeconds>();
		Grammar.DefinePostUnaryOperator<FFramesPerSecond>();
		Grammar.DefinePostUnaryOperator<FNonDropFrame>();
		Grammar.DefinePostUnaryOperator<FDropFrame>();

		JumpTable.MapPostUnary<FSeconds>(
			[](double In) -> FExpressionResult
			{
				if (FMath::RoundToDouble(In) == In && In > 0.0 && In < TNumericLimits<int32>::Max())
				{
					return MakeValue(FFrameRate(1, static_cast<int32>(In)));
				}
				return MakeFrameRateFromInterval(In);
			}
		);
		JumpTable.MapPostUnary<FFramesPerSecond>([](double In) -> FExpressionResult { return MakeFrameRateFromFPS(In); });
		JumpTable.MapPostUnary<FNonDropFrame>([](double In) -> FExpressionResult { return MakeFrameRateFromFPS(In); });
		JumpTable.MapPostUnary<FDropFrame>([](double In) -> FExpressionResult {
			if (!FMath::IsNearlyEqual(In, NTSC_30_FPS, UE_DOUBLE_KINDA_SMALL_NUMBER) && !FMath::IsNearlyEqual(In, NTSC_60_FPS, UE_DOUBLE_KINDA_SMALL_NUMBER))
			{
				return MakeError(FText::Format(LOCTEXT("InvalidDropFrameFPS", "Drop frame not supported for FPS specified: {0}.\nDrop frame timecode is only supported for NTSC_30 ({1}fps) and NTSC_60 ({2}fps) frame rates."), In, NTSC_30_FPS, NTSC_60_FPS));
			}

			return MakeFrameRateFromFPS(In);
		});
		JumpTable.MapBinary<FForwardSlash>([](double A, double B) -> FExpressionResult { return MakeFrameRate(A, B); });
	}

	TValueOrError<FFrameRate, FExpressionError> Evaluate(const TCHAR* InExpression) const
	{
		using namespace ExpressionParser;

		TValueOrError<TArray<FExpressionToken>, FExpressionError> LexResult = ExpressionParser::Lex(InExpression, TokenDefinitions);
		if (!LexResult.IsValid())
		{
			return MakeError(LexResult.StealError());
		}

		TValueOrError<TArray<FCompiledToken>, FExpressionError> CompilationResult = ExpressionParser::Compile(LexResult.StealValue(), Grammar);
		if (!CompilationResult.IsValid())
		{
			return MakeError(CompilationResult.StealError());
		}

		TOperatorEvaluationEnvironment<> Env(JumpTable, nullptr);
		TValueOrError<FExpressionNode, FExpressionError> Result = ExpressionParser::Evaluate(CompilationResult.GetValue(), Env);
		if (!Result.IsValid())
		{
			return MakeError(Result.GetError());
		}

		FExpressionNode& Node = Result.GetValue();

		if (const double* Number = Node.Cast<double>())
		{
			FExpressionResult ParseResult = (*Number > 1) ? MakeFrameRateFromFPS(*Number) : MakeFrameRateFromInterval(*Number);
			if (!ParseResult.IsValid())
			{
				return MakeError(ParseResult.StealError());
			}
			else if (const FFrameRate* FrameRate = ParseResult.GetValue().Cast<FFrameRate>())
			{
				return MakeValue(*FrameRate);
			}
		}
		else if (const FFrameRate* FrameRate = Node.Cast<FFrameRate>())
		{
			return MakeValue(*FrameRate);
		}

		return MakeError(LOCTEXT("UnrecognizedResult", "Unrecognized result returned from expression"));
	}

	static FFrameRateParser& Get()
	{
		static FFrameRateParser StaticFrameRateParser;
		return StaticFrameRateParser;
	}

private:

	static FExpressionResult MakeFrameRate(double A, double B)
	{
		double MaxInteger = TNumericLimits<int32>::Max();
		double IntPartA = 0.0, IntPartB = 0.0;

		if (A <= 0.0 || FMath::Modf(A, &IntPartA) != 0.0 || IntPartA > MaxInteger)
		{
			return MakeError(FText::Format(LOCTEXT("InvalidNumerator", "Invalid framerate numerator: {0}"), A));
		}
		else if (B <= 0.0 || FMath::Modf(B, &IntPartB) != 0.0 || IntPartB > MaxInteger)
		{
			return MakeError(FText::Format(LOCTEXT("InvalidDenominator", "Invalid framerate denominator: {0}"), B));
		}

		return MakeValue(FFrameRate(static_cast<int32>(IntPartA), static_cast<int32>(IntPartB)));
	}

	static FExpressionResult MakeFrameRateFromFPS(double InFPS)
	{
		if (InFPS <= 0.0 || InFPS >= TNumericLimits<int32>::Max())
		{
			return MakeError(FText::Format(LOCTEXT("OutOfBoundsFPS", "Invalid FPS specified: {0} (out of bounds)"), InFPS));
		}

		double RoundedFPS = FMath::RoundToDouble(InFPS);
		if (RoundedFPS != InFPS)
		{
			// Allow creating FFrameRates for fractional FPS values only if they match an NTSC frame rate.
			if (FMath::IsNearlyEqual(InFPS, NTSC_24_FPS, UE_DOUBLE_KINDA_SMALL_NUMBER))
			{
				return MakeValue(FFrameRate(NTSC_24_Numerator, NTSC_Denominator));
			}
			else if (FMath::IsNearlyEqual(InFPS, NTSC_30_FPS, UE_DOUBLE_KINDA_SMALL_NUMBER))
			{
				return MakeValue(FFrameRate(NTSC_30_Numerator, NTSC_Denominator));
			}
			else if (FMath::IsNearlyEqual(InFPS, NTSC_60_FPS, UE_DOUBLE_KINDA_SMALL_NUMBER))
			{
				return MakeValue(FFrameRate(NTSC_60_Numerator, NTSC_Denominator));
			}

			return MakeError(FText::Format(LOCTEXT("FractionalFrameRate_Format", "Fractional FPS specified: {0}.\nPlease use x/y notation to define such framerates."), InFPS));
		}

		return MakeValue(FFrameRate(static_cast<int32>(RoundedFPS), 1));
	}

	static FExpressionResult MakeFrameRateFromInterval(double InSecondInterval)
	{
		if (InSecondInterval <= 0.0)
		{
			return MakeError(FText::Format(LOCTEXT("InvalidInterval", "Invalid interval specified: {0}"), InSecondInterval));
		}

		return MakeFrameRateFromFPS(1.0 / InSecondInterval);
	}

	FTokenDefinitions TokenDefinitions;
	FExpressionGrammar Grammar;
	FOperatorJumpTable JumpTable;

};

double FFrameRate::MaxSeconds() const
{
	return FFrameNumber(TNumericLimits<int32>::Max()) / *this;
}

FText FFrameRate::ToPrettyText() const
{
	double FPS = AsDecimal();
	if (FPS > 1)
	{
		// The localization system will create "24,000" out of a framerate of 24000.00. This presents an issue when we
		// try to consume the resulting format string as the localization system currently has grouping separators disabled.
		// We'll format the text by hand (instead of falling back to default FText::Format) to omit the grouping separator
		// which will allow the text to properly round trip through UI fields.
		return FText::Format(NSLOCTEXT("FFrameRate", "FPS_Format", "{0} fps"), FText::AsNumber(FPS, &FNumberFormattingOptions::DefaultNoGrouping()));
	}
	else
	{
		return FText::Format(NSLOCTEXT("FFrameRate", "Seconds_Format", "{0} s"), 1.0/FPS);
	}
}

bool FFrameRate::ComputeGridSpacing(float PixelsPerSecond, double& OutMajorInterval, int32& OutMinorDivisions, float MinTickPx, float DesiredMajorTickPx) const
{
	if (PixelsPerSecond <= 0.f)
	{
		return false;
	}

	const int32 RoundedFPS = static_cast<int32>(FMath::Clamp(FMath::RoundToDouble(AsDecimal()), 0.0, static_cast<double>(TNumericLimits<int32>::Max())));

	// Start showing time on second boundaries after we can represent 0.5s (60 ^ -0.169 ~= 0.5)
	static float TimeDisplayThresholdExponent = -0.169f;
	const float  TimeExponent = FMath::LogX(60.f, MinTickPx / PixelsPerSecond);

	if (TimeExponent >= TimeDisplayThresholdExponent)
	{
		const float TimeOrder = FMath::Pow(60.f, FMath::FloorToFloat(FMath::LogX(60.f, DesiredMajorTickPx / PixelsPerSecond)));

		// Showing hours, minutes or seconds
		static const int32 DesirableBases[]  = { 1, 2, 5, 10, 30, 60 };
		static const int32 NumDesirableBases = UE_ARRAY_COUNT(DesirableBases);

		const float ScaleFloat= FMath::CeilToFloat(DesiredMajorTickPx / PixelsPerSecond / TimeOrder);
		const int32 Scale = FMath::TruncToInt(ScaleFloat);
		const int32 BaseIndex = FMath::Min(static_cast<int32>(Algo::LowerBound(DesirableBases, Scale)), NumDesirableBases-1);

		const int32 Base = DesirableBases[BaseIndex];
		const float MajorIntervalSecondsFloat = FMath::RoundToFloat(FMath::Pow(static_cast<float>(Base), FMath::CeilToFloat(FMath::LogX(static_cast<float>(Base), ScaleFloat))));
		const int32 MajorIntervalSeconds = FMath::TruncToInt(MajorIntervalSecondsFloat);

		OutMajorInterval  = TimeOrder * MajorIntervalSecondsFloat;
		OutMinorDivisions = static_cast<int32>(FMath::Min(FMath::RoundUpToPowerOfTwo(static_cast<uint32>(OutMajorInterval / (MinTickPx / PixelsPerSecond))), static_cast<uint32>(TNumericLimits<int32>::Max())));

		// Find the lowest number of divisions we can show that's larger than the minimum tick size
		OutMinorDivisions = 0;
		for (int32 DivIndex = 0; DivIndex < BaseIndex; ++DivIndex)
		{
			if (Base % DesirableBases[DivIndex] == 0)
			{
				int32 MinorDivisions = MajorIntervalSeconds/DesirableBases[DivIndex];
				if (OutMajorInterval / MinorDivisions * PixelsPerSecond >= MinTickPx)
				{
					OutMinorDivisions = MinorDivisions;
					break;
				}
			}
		}
	}
	else if (RoundedFPS > 0)
	{
		// Showing frames
		TArray<int32, TInlineAllocator<10>> CommonBases;

		// Divide the rounded frame rate by 2s, 3s or 5s recursively
		{
			int32 LowestBase = RoundedFPS;
			for (;;)
			{
				CommonBases.Add(LowestBase);
	
				if (LowestBase % 2 == 0)      { LowestBase = LowestBase / 2; }
				else if (LowestBase % 3 == 0) { LowestBase = LowestBase / 3; }
				else if (LowestBase % 5 == 0) { LowestBase = LowestBase / 5; }
				else                          { break; }
			}
		}

		Algo::Reverse(CommonBases);

		const float ScaleFloat= FMath::CeilToFloat(static_cast<float>(DesiredMajorTickPx / PixelsPerSecond * AsDecimal()));
		const int32 Scale     = FMath::TruncToInt(ScaleFloat);
		const int32 BaseIndex = FMath::Min(Algo::LowerBound(CommonBases, Scale), CommonBases.Num()-1);
		const int32 Base      = CommonBases[BaseIndex];

		int32 MajorIntervalFrames = FMath::CeilToInt(ScaleFloat / float(Base)) * Base;
		OutMajorInterval  = MajorIntervalFrames * AsInterval();

		// Find the lowest number of divisions we can show that's larger than the minimum tick size
		OutMinorDivisions = 0;
		for (int32 DivIndex = 0; DivIndex < BaseIndex; ++DivIndex)
		{
			if (Base % CommonBases[DivIndex] == 0)
			{
				int32 MinorDivisions = MajorIntervalFrames/CommonBases[DivIndex];
				if (OutMajorInterval / MinorDivisions * PixelsPerSecond >= MinTickPx)
				{
					OutMinorDivisions = MinorDivisions;
					break;
				}
			}
		}
	}
	else
	{
		// Showing ms etc
		const float TimeOrder = FMath::Pow(10.f, FMath::FloorToFloat(FMath::LogX(10.f, DesiredMajorTickPx / PixelsPerSecond)));
		const float Scale = FMath::CeilToFloat(DesiredMajorTickPx / PixelsPerSecond / TimeOrder);

		static float RoundToBase = 5.f;
		OutMajorInterval  = TimeOrder * FMath::Pow(RoundToBase, FMath::CeilToFloat(FMath::LogX(RoundToBase, Scale)));
		OutMinorDivisions = FMath::RoundUpToPowerOfTwo(static_cast<uint32>(OutMajorInterval / (MinTickPx / PixelsPerSecond)));
	}

	return OutMajorInterval != 0;
}

TValueOrError<FFrameRate, FExpressionError> ParseFrameRate(const TCHAR* FrameRateString)
{
	FFrameRateParser& StaticFrameRateParser = FFrameRateParser::Get();
	return StaticFrameRateParser.Evaluate(FrameRateString);
}

bool TryParseString(FFrameRate& OutFrameRate, const TCHAR* InString)
{
	FFrameRateParser& StaticFrameRateParser = FFrameRateParser::Get();
	TValueOrError<FFrameRate, FExpressionError> ParseResult = StaticFrameRateParser.Evaluate(InString);
	if (ParseResult.IsValid())
	{
		OutFrameRate = ParseResult.GetValue();
		return true;
	}

	return false;
}

FArchive& operator<<(FArchive& Ar, FFrameRate& FrameRate)
{
	Ar << FrameRate.Numerator;
	Ar << FrameRate.Denominator;

	return Ar;
}

bool FFrameRate::Serialize(FArchive& Ar)
{
	Ar << Numerator;
	Ar << Denominator;
	
	return true;
}

#undef LOCTEXT_NAMESPACE