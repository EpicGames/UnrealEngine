// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/Guid.h"

#include "Algo/Replace.h"
#include "Containers/ArrayView.h"
#include "Containers/StaticArray.h"
#include "HAL/PlatformMisc.h"
#include "HAL/PlatformString.h"
#include "HAL/UnrealMemory.h"
#include "Hash/Blake3.h"
#include "Math/Int128.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/Base64.h"
#include "Misc/ByteSwap.h"
#include "Misc/CString.h"
#include "Misc/Char.h"
#include "Misc/Parse.h"
#include "Misc/StringBuilder.h"
#include "Serialization/StructuredArchiveNameHelpers.h"
#include "Serialization/StructuredArchiveSlots.h"
#include "Templates/AlignmentTemplates.h"
#include "UObject/PropertyPortFlags.h"


/* FGuid interface
 *****************************************************************************/

bool FGuid::ExportTextItem(FString& ValueStr, FGuid const& DefaultValue, UObject* Parent, int32 PortFlags, class UObject* ExportRootScope) const
{
	AppendString(ValueStr, EGuidFormats::Digits);

	return true;
}


bool FGuid::ImportTextItem(const TCHAR*& Buffer, int32 PortFlags, class UObject* Parent, FOutputDevice* ErrorText)
{
	constexpr int32 GuidStrLen = 32;

	if (FPlatformString::Strnlen(Buffer, GuidStrLen) < GuidStrLen)
	{
		return false;
	}

	if (!ParseExact(FStringView(Buffer, GuidStrLen), EGuidFormats::Digits, *this))
	{
		return false;
	}

	Buffer += GuidStrLen;

	return true;
}


void FGuid::AppendString(FString& Out, EGuidFormats Format) const
{
	switch (Format)
	{
	case EGuidFormats::DigitsWithHyphens:
		Out.Appendf(TEXT("%08X-%04X-%04X-%04X-%04X%08X"), A, B >> 16, B & 0xFFFF, C >> 16, C & 0xFFFF, D);
		return;

	case EGuidFormats::DigitsWithHyphensLower:
		Out.Appendf(TEXT("%08x-%04x-%04x-%04x-%04x%08x"), A, B >> 16, B & 0xFFFF, C >> 16, C & 0xFFFF, D);
		return;

	case EGuidFormats::DigitsWithHyphensInBraces:
		Out.Appendf(TEXT("{%08X-%04X-%04X-%04X-%04X%08X}"), A, B >> 16, B & 0xFFFF, C >> 16, C & 0xFFFF, D);
		return;

	case EGuidFormats::DigitsWithHyphensInParentheses:
		Out.Appendf(TEXT("(%08X-%04X-%04X-%04X-%04X%08X)"), A, B >> 16, B & 0xFFFF, C >> 16, C & 0xFFFF, D);
		return;

	case EGuidFormats::HexValuesInBraces:
		Out.Appendf(TEXT("{0x%08X,0x%04X,0x%04X,{0x%02X,0x%02X,0x%02X,0x%02X,0x%02X,0x%02X,0x%02X,0x%02X}}"), A, B >> 16, B & 0xFFFF, C >> 24, (C >> 16) & 0xFF, (C >> 8) & 0xFF, C & 0XFF, D >> 24, (D >> 16) & 0XFF, (D >> 8) & 0XFF, D & 0XFF);
		return;

	case EGuidFormats::UniqueObjectGuid:
		Out.Appendf(TEXT("%08X-%08X-%08X-%08X"), A, B, C, D);
		return;

	case EGuidFormats::Short:
	{
		uint32 Bytes[4] = { NETWORK_ORDER32(A), NETWORK_ORDER32(B), NETWORK_ORDER32(C), NETWORK_ORDER32(D) };
		TCHAR Buffer[25];
		int32 Len = FBase64::Encode(reinterpret_cast<const uint8*>(Bytes), sizeof(Bytes), Buffer);
		TArrayView<TCHAR> Result(Buffer, Len);
		
		Algo::Replace(Result, '+', '-');
		Algo::Replace(Result, '/', '_');

		// Remove trailing '=' base64 padding
		check(Len == 24);
		Result.LeftChopInline(2);

		Out.Append(Result);
		return;
	}

	case EGuidFormats::Base36Encoded:
	{
		static const uint8 Alphabet[36] = 
		{
			'0', '1', '2', '3', '4', '5', '6', '7', '8', '9', 'A', 'B', 'C', 'D', 'E', 'F',
			'G', 'H', 'I', 'J', 'K', 'L', 'M', 'N', 'O', 'P', 'Q', 'R', 'S', 'T', 'U', 'V',
			'W', 'X', 'Y', 'Z'
		};

		FUInt128 Value(A, B, C, D);

		TStaticArray<TCHAR, 25> Result;
		for (int32 I = 24; I >= 0; --I)
		{
			uint32 Remainder;
			Value = Value.Divide(36, Remainder);
			Result[I] = Alphabet[Remainder];
		}

		Out.Append(Result);
		return;
	}

	case EGuidFormats::DigitsLower:
		Out.Appendf(TEXT("%08x%08x%08x%08x"), A, B, C, D);
		return;

	default:
	{
		uint32 Bytes[4] = { NETWORK_ORDER32(A), NETWORK_ORDER32(B), NETWORK_ORDER32(C), NETWORK_ORDER32(D) };
		BytesToHex(reinterpret_cast<const uint8*>(Bytes), sizeof(Bytes), Out);
		return;
	}
	}
}

void FGuid::AppendString(FAnsiStringBuilderBase& Builder, EGuidFormats Format) const
{
	switch (Format)
	{
	case EGuidFormats::DigitsWithHyphens:
		Builder.Appendf("%08X-%04X-%04X-%04X-%04X%08X", A, B >> 16, B & 0xFFFF, C >> 16, C & 0xFFFF, D);
		return;

	case EGuidFormats::DigitsWithHyphensLower:
		Builder.Appendf("%08x-%04x-%04x-%04x-%04x%08x", A, B >> 16, B & 0xFFFF, C >> 16, C & 0xFFFF, D);
		return;

	case EGuidFormats::DigitsWithHyphensInBraces:
		Builder.Appendf("{%08X-%04X-%04X-%04X-%04X%08X}", A, B >> 16, B & 0xFFFF, C >> 16, C & 0xFFFF, D);
		return;

	case EGuidFormats::DigitsWithHyphensInParentheses:
		Builder.Appendf("(%08X-%04X-%04X-%04X-%04X%08X)", A, B >> 16, B & 0xFFFF, C >> 16, C & 0xFFFF, D);
		return;

	case EGuidFormats::HexValuesInBraces:
		Builder.Appendf("{0x%08X,0x%04X,0x%04X,{0x%02X,0x%02X,0x%02X,0x%02X,0x%02X,0x%02X,0x%02X,0x%02X}}",
			A, B >> 16, B & 0xFFFF, C >> 24, (C >> 16) & 0xFF, (C >> 8) & 0xFF, C & 0XFF, D >> 24, (D >> 16) & 0XFF, (D >> 8) & 0XFF, D & 0XFF);
		return;

	case EGuidFormats::UniqueObjectGuid:
		Builder.Appendf("%08X-%08X-%08X-%08X", A, B, C, D);
		return;

	case EGuidFormats::Short:
	{
		uint32 Bytes[4] = { NETWORK_ORDER32(A), NETWORK_ORDER32(B), NETWORK_ORDER32(C), NETWORK_ORDER32(D) };
		int32 Index = Builder.AddUninitialized(24);
		TArrayView<ANSICHAR> Result(Builder.GetData() + Index, 24);
		int32 Len = FBase64::Encode(reinterpret_cast<const uint8*>(Bytes), sizeof(Bytes), Result.GetData());

		Algo::Replace(Result, '+', '-');
		Algo::Replace(Result, '/', '_');

		// Remove trailing '=' base64 padding
		check(Len == 24);
		Builder.RemoveSuffix(2);
		return;
	}

	case EGuidFormats::Base36Encoded:
	{
		static const uint8 Alphabet[36] =
		{
			'0', '1', '2', '3', '4', '5', '6', '7', '8', '9', 'A', 'B', 'C', 'D', 'E', 'F',
			'G', 'H', 'I', 'J', 'K', 'L', 'M', 'N', 'O', 'P', 'Q', 'R', 'S', 'T', 'U', 'V',
			'W', 'X', 'Y', 'Z'
		};

		FUInt128 Value(A, B, C, D);

		int32 Index = Builder.AddUninitialized(25);
		TArrayView<ANSICHAR> Result(Builder.GetData() + Index, 25);

		for (int32 I = 24; I >= 0; --I)
		{
			uint32 Remainder;
			Value = Value.Divide(36, Remainder);
			Result[I] = Alphabet[Remainder];
		}

		return;
	}

	case EGuidFormats::DigitsLower:
		Builder.Appendf("%08x%08x%08x%08x", A, B, C, D);
		return;

	default:
		Builder.Appendf("%08X%08X%08X%08X", A, B, C, D);
		return;
	}
}

void FGuid::AppendString(FUtf8StringBuilderBase& Builder, EGuidFormats Format) const
{
	TAnsiStringBuilder<70> AnsiBuilder;
	AppendString(AnsiBuilder, Format);
	Builder.Append(AnsiBuilder);
}

void FGuid::AppendString(FWideStringBuilderBase& Builder, EGuidFormats Format) const
{
	TAnsiStringBuilder<70> AnsiBuilder;
	AppendString(AnsiBuilder, Format);
	Builder.Append(AnsiBuilder);
}


/* FGuid static interface
 *****************************************************************************/

FGuid FGuid::NewGuid()
{
	FGuid Result(0, 0, 0, 0);
	FPlatformMisc::CreateGuid(Result);

	return Result;
}

FGuid FGuid::NewDeterministicGuid(FStringView ObjectPath, uint64 Seed)
{
	// Convert the objectpath to utf8 so that whether TCHAR is UTF8 or UTF16 does not alter the hash.
	TUtf8StringBuilder<1024> Utf8ObjectPath(InPlace, ObjectPath);

	FBlake3 Builder;

	// Hash this as the namespace of the Version 3 UUID, to avoid collisions with any other guids created using Blake3.
	static FGuid BaseVersion(TEXT("182b8dd3-f963-477f-a57d-70a339d922d8"));
	Builder.Update(&BaseVersion, sizeof(FGuid));
	Builder.Update(&Seed, sizeof(Seed));
	Builder.Update(Utf8ObjectPath.GetData(), Utf8ObjectPath.Len() * sizeof(UTF8CHAR));

	FBlake3Hash Hash = Builder.Finalize();

	return NewGuidFromHash(Hash);
}

FGuid FGuid::NewGuidFromHash(const FBlake3Hash& Hash)
{
	// We use the first 16 bytes of the BLAKE3 hash to create the guid, there is no specific reason why these were
	// chosen, we could take any pattern or combination of bytes.
	return NewGuidFromHashBytes(&Hash.GetBytes(), sizeof(Hash.GetBytes()));
}

FGuid FGuid::NewGuidFromHashBytes(const void* HashData, int64 DataLen)
{
	alignas(uint32) uint8 Buffer[16];
	if (DataLen < 16 || !IsAligned(HashData, alignof(uint32)))
	{
		FMemory::Memcpy(Buffer, HashData, FMath::Min(DataLen, 16));
		if (DataLen < 16)
		{
			FMemory::Memzero(&Buffer[DataLen], 16 - DataLen);
		}
		HashData = Buffer;
		DataLen = 16;
	}

	const uint32* HashInts = reinterpret_cast<const uint32*>(HashData);
	uint32 A = uint32(HashInts[0]);
	uint32 B = uint32(HashInts[1]);
	uint32 C = uint32(HashInts[2]);
	uint32 D = uint32(HashInts[3]);

	// Convert to a Variant 1 Version 3 UUID, which is meant to be created by hashing the namespace and name with MD5.
	// We use that version even though we're using BLAKE3 instead of MD5 because we don't have a better alternative.
	// It is still very useful for avoiding collisions with other UUID variants.
	B = (B & ~0x0000f000) | 0x00003000; // Version 3 (MD5)
	C = (C & ~0xc0000000) | 0x80000000; // Variant 1 (RFC 4122 UUID)
	return FGuid(A, B, C, D);
}

FGuid FGuid::Combine(const FGuid& GuidA, const FGuid& GuidB)
{
	return FGuid(
		HashCombine(GuidA.A, GuidB.A),
		HashCombine(GuidA.B, GuidB.B),
		HashCombine(GuidA.C, GuidB.C),
		HashCombine(GuidA.D, GuidB.D)
	);
}

FGuid FGuid::CombineValid(const FGuid& GuidA, const FGuid& GuidB)
{
	if (GuidA.IsValid())
	{
		if (GuidB.IsValid())
		{
			return Combine(GuidA, GuidB);
		}

		return GuidA;
	}

	return GuidB;
}

bool FGuid::Parse(const TCHAR* GuidString, FGuid& OutGuid)
{
	return Parse(FStringView(GuidString), OutGuid);
}

bool FGuid::Parse(const FString& GuidString, FGuid& OutGuid)
{
	return Parse(FStringView(GuidString), OutGuid);
}

bool FGuid::Parse(FStringView GuidString, FGuid& OutGuid)
{
	if (GuidString.Len() == 32)
	{
		return ParseExact(GuidString, EGuidFormats::Digits, OutGuid);
	}

	if (GuidString.Len() == 36)
	{
		return ParseExact(GuidString, EGuidFormats::DigitsWithHyphens, OutGuid);
	}

	if (GuidString.Len() == 38)
	{
		if (GuidString[0] == TEXT('{'))
		{
			return ParseExact(GuidString, EGuidFormats::DigitsWithHyphensInBraces, OutGuid);
		}

		return ParseExact(GuidString, EGuidFormats::DigitsWithHyphensInParentheses, OutGuid);
	}

	if (GuidString.Len() == 68)
	{
		return ParseExact(GuidString, EGuidFormats::HexValuesInBraces, OutGuid);
	}

	if (GuidString.Len() == 35)
	{
		return ParseExact(GuidString, EGuidFormats::UniqueObjectGuid, OutGuid);
	}

	if (GuidString.Len() == 22)
	{
		return ParseExact(GuidString, EGuidFormats::Short, OutGuid);
	}

	if (GuidString.Len() == 25)
	{
		return ParseExact(GuidString, EGuidFormats::Base36Encoded, OutGuid);
	}

	return false;
}

bool FGuid::ParseExact(const TCHAR* GuidString, EGuidFormats Format, FGuid& OutGuid)
{
	return ParseExact(FStringView(GuidString), Format, OutGuid);
}

bool FGuid::ParseExact(const FString& GuidString, EGuidFormats Format, FGuid& OutGuid)
{
	return ParseExact(FStringView(GuidString), Format, OutGuid);
}

bool FGuid::ParseExact(FStringView GuidString, EGuidFormats Format, FGuid& OutGuid)
{
	// The below normalizing into the Digits format doesn't work with Short and Base36 Guids
	if (Format == EGuidFormats::Short)
	{
		if (GuidString.Len() != 22)
		{
			return false;
		}

		uint32 Data[4] = {};

		// We don't need to do replacements to get an accurate size (deferring allocations till later)
		if (FBase64::GetDecodedDataSize(GuidString.GetData(), GuidString.Len()) != sizeof(Data))
		{
			// This isn't a Short GUID if it's not 128 bits / 16 bytes (the size of Data)
			return false;
		}

		// Replace the characters we replaced going out (but not the padding, UE discards it immediately)
		TStringBuilder<22> GuidCopy;
		GuidCopy += GuidString;
		Algo::Replace(MakeArrayView(GuidCopy), TEXT('-'), TEXT('+'));
		Algo::Replace(MakeArrayView(GuidCopy), TEXT('_'), TEXT('/'));

		// Decode the data
		if (!FBase64::Decode(GuidCopy.GetData(), GuidCopy.Len(), reinterpret_cast<uint8*>(&Data)))
		{
			// Data is not valid in some way
			return false;
		}

		OutGuid = FGuid(NETWORK_ORDER32(Data[0]),
						NETWORK_ORDER32(Data[1]),
						NETWORK_ORDER32(Data[2]),
						NETWORK_ORDER32(Data[3]));
		return true;
	}

	if (Format == EGuidFormats::Base36Encoded)
	{
		if (GuidString.Len() != 25)
		{
			return false;
		}

		FUInt128 Value;
		for (const TCHAR GuidChar : GuidString)
		{
			Value *= 36;
			if (GuidChar >= '0' && GuidChar <= '9')
			{
				Value += GuidChar - '0';
			}
			else if (GuidChar >= 'A' && GuidChar <= 'Z')
			{
				Value += GuidChar - 'A' + 10;
			}
			else
			{
				return false;
			}
		}

		OutGuid = FGuid(
			Value.GetQuadPart(3),
			Value.GetQuadPart(2),
			Value.GetQuadPart(1),
			Value.GetQuadPart(0)
		);

		return true;
	}

	TStringBuilder<32> NormalizedGuidString;

	if (Format == EGuidFormats::Digits || Format == EGuidFormats::DigitsLower)
	{
		if (GuidString.Len() != 32)
		{
			return false;
		}

		NormalizedGuidString = GuidString;
	}
	else if (Format == EGuidFormats::DigitsWithHyphens || Format == EGuidFormats::DigitsWithHyphensLower)
	{
		if (GuidString.Len() != 36)
		{
			return false;
		}

		if ((GuidString[8] != TCHAR('-')) ||
			(GuidString[13] != TCHAR('-')) ||
			(GuidString[18] != TCHAR('-')) ||
			(GuidString[23] != TCHAR('-')))
		{
			return false;
		}

		NormalizedGuidString += GuidString.Mid(0, 8);
		NormalizedGuidString += GuidString.Mid(9, 4);
		NormalizedGuidString += GuidString.Mid(14, 4);
		NormalizedGuidString += GuidString.Mid(19, 4);
		NormalizedGuidString += GuidString.Mid(24, 12);
	}
	else if (Format == EGuidFormats::DigitsWithHyphensInBraces)
	{
		if (GuidString.Len() != 38)
		{
			return false;
		}

		if ((GuidString[0] != TCHAR('{')) ||
			(GuidString[9] != TCHAR('-')) ||
			(GuidString[14] != TCHAR('-')) ||
			(GuidString[19] != TCHAR('-')) ||
			(GuidString[24] != TCHAR('-')) ||
			(GuidString[37] != TCHAR('}')))
		{
			return false;
		}

		NormalizedGuidString += GuidString.Mid(1, 8);
		NormalizedGuidString += GuidString.Mid(10, 4);
		NormalizedGuidString += GuidString.Mid(15, 4);
		NormalizedGuidString += GuidString.Mid(20, 4);
		NormalizedGuidString += GuidString.Mid(25, 12);
	}
	else if (Format == EGuidFormats::DigitsWithHyphensInParentheses)
	{
		if (GuidString.Len() != 38)
		{
			return false;
		}

		if ((GuidString[0] != TCHAR('(')) ||
			(GuidString[9] != TCHAR('-')) ||
			(GuidString[14] != TCHAR('-')) ||
			(GuidString[19] != TCHAR('-')) ||
			(GuidString[24] != TCHAR('-')) ||
			(GuidString[37] != TCHAR(')')))
		{
			return false;
		}

		NormalizedGuidString += GuidString.Mid(1, 8);
		NormalizedGuidString += GuidString.Mid(10, 4);
		NormalizedGuidString += GuidString.Mid(15, 4);
		NormalizedGuidString += GuidString.Mid(20, 4);
		NormalizedGuidString += GuidString.Mid(25, 12);
	}
	else if (Format == EGuidFormats::HexValuesInBraces)
	{
		if (GuidString.Len() != 68)
		{
			return false;
		}

		if ((GuidString[0] != TCHAR('{')) ||
			(GuidString[1] != TCHAR('0')) ||
			(GuidString[2] != TCHAR('x')) ||
			(GuidString[11] != TCHAR(',')) ||
			(GuidString[12] != TCHAR('0')) ||
			(GuidString[13] != TCHAR('x')) ||
			(GuidString[18] != TCHAR(',')) ||
			(GuidString[19] != TCHAR('0')) ||
			(GuidString[20] != TCHAR('x')) ||
			(GuidString[25] != TCHAR(',')) ||
			(GuidString[26] != TCHAR('{')) ||
			(GuidString[27] != TCHAR('0')) ||
			(GuidString[28] != TCHAR('x')) ||
			(GuidString[31] != TCHAR(',')) ||
			(GuidString[32] != TCHAR('0')) ||
			(GuidString[33] != TCHAR('x')) ||
			(GuidString[36] != TCHAR(',')) ||
			(GuidString[37] != TCHAR('0')) ||
			(GuidString[38] != TCHAR('x')) ||
			(GuidString[41] != TCHAR(',')) ||
			(GuidString[42] != TCHAR('0')) ||
			(GuidString[43] != TCHAR('x')) ||
			(GuidString[46] != TCHAR(',')) ||
			(GuidString[47] != TCHAR('0')) ||
			(GuidString[48] != TCHAR('x')) ||
			(GuidString[51] != TCHAR(',')) ||
			(GuidString[52] != TCHAR('0')) ||
			(GuidString[53] != TCHAR('x')) ||
			(GuidString[56] != TCHAR(',')) ||
			(GuidString[57] != TCHAR('0')) ||
			(GuidString[58] != TCHAR('x')) ||
			(GuidString[61] != TCHAR(',')) ||
			(GuidString[62] != TCHAR('0')) ||
			(GuidString[63] != TCHAR('x')) ||
			(GuidString[66] != TCHAR('}')) ||
			(GuidString[67] != TCHAR('}')))
		{
			return false;
		}

		NormalizedGuidString += GuidString.Mid(3, 8);
		NormalizedGuidString += GuidString.Mid(14, 4);
		NormalizedGuidString += GuidString.Mid(21, 4);
		NormalizedGuidString += GuidString.Mid(29, 2);
		NormalizedGuidString += GuidString.Mid(34, 2);
		NormalizedGuidString += GuidString.Mid(39, 2);
		NormalizedGuidString += GuidString.Mid(44, 2);
		NormalizedGuidString += GuidString.Mid(49, 2);
		NormalizedGuidString += GuidString.Mid(54, 2);
		NormalizedGuidString += GuidString.Mid(59, 2);
		NormalizedGuidString += GuidString.Mid(64, 2);
	}
	else if (Format == EGuidFormats::UniqueObjectGuid)
	{
		if (GuidString.Len() != 35)
		{
			return false;
		}

		if ((GuidString[8] != TCHAR('-')) ||
			(GuidString[17] != TCHAR('-')) ||
			(GuidString[26] != TCHAR('-')))
		{
			return false;
		}

		NormalizedGuidString += GuidString.Mid(0, 8);
		NormalizedGuidString += GuidString.Mid(9, 8);
		NormalizedGuidString += GuidString.Mid(18, 8);
		NormalizedGuidString += GuidString.Mid(27, 8);
	}

	FStringView NormalizedGuidStringView = NormalizedGuidString.ToView();
	for (const TCHAR GuidChar : NormalizedGuidStringView)
	{
		if (!FChar::IsHexDigit(GuidChar))
		{
			return false;
		}
	}

	OutGuid = FGuid(
		FParse::HexNumber(NormalizedGuidStringView.Mid(0, 8)),
		FParse::HexNumber(NormalizedGuidStringView.Mid(8, 8)),
		FParse::HexNumber(NormalizedGuidStringView.Mid(16, 8)),
		FParse::HexNumber(NormalizedGuidStringView.Mid(24, 8))
	);

	return true;
}

FArchive& operator<<(FArchive& Ar, FGuid& G)
{
	return Ar << G.A << G.B << G.C << G.D;
}

void operator<<(FStructuredArchive::FSlot Slot, FGuid& G)
{
	const FArchiveState& State = Slot.GetArchiveState();
	if (State.IsTextFormat())
	{
		if (State.IsLoading())
		{
			FString AsString;
			Slot << AsString;
			LexFromString(G, *AsString);
		}
		else
		{
			FString AsString = LexToString(G);
			Slot << AsString;
		}
	}
	else
	{
		FStructuredArchive::FRecord Record = Slot.EnterRecord();
		Record << SA_VALUE(TEXT("A"), G.A) << SA_VALUE(TEXT("B"), G.B) << SA_VALUE(TEXT("C"), G.C) << SA_VALUE(TEXT("D"), G.D);
	}
}
