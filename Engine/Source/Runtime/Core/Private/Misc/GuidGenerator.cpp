// Copyright Epic Games, Inc. All Rights Reserved.


/*
** Copyright (c) 1990- 1993, 1996 Open Software Foundation, Inc.
** Copyright (c) 1989 by Hewlett-Packard Company, Palo Alto, Ca. &
** Digital Equipment Corporation, Maynard, Mass.
** To anyone who acknowledges that this file is provided "AS IS"
** without any express or implied warranty: permission to use, copy,
** modify, and distribute this file for any purpose is hereby
** granted without fee, provided that the above copyright notices and
** this notice appears in all source code copies, and that none of
** the names of Open Software Foundation, Inc., Hewlett-Packard
** Company, or Digital Equipment Corporation be used in advertising
** or publicity pertaining to distribution of the software without
** specific, written prior permission.  Neither Open Software
** Foundation, Inc., Hewlett-Packard Company, nor Digital Equipment
** Corporation makes any representations about the suitability of
** this software for any purpose.
*/

#include "CoreTypes.h"

#ifdef __GNUC__

#include <sys/types.h>
#include <sys/time.h>
#include <unistd.h>
#include <string.h>

/*----------------------------------------------------------------------------
	Private definitions.
----------------------------------------------------------------------------*/

typedef unsigned long   unsigned32;
typedef unsigned short  unsigned16;
typedef unsigned char   unsigned8;
typedef unsigned char   byte;

#define CLOCK_SEQ_LAST	0x3FFF
#define RAND_MASK		CLOCK_SEQ_LAST

typedef struct _UUID_t {
  unsigned32          time_low;
  unsigned16          time_mid;
  unsigned16          time_hi_and_version;
  unsigned8           clock_seq_hi_and_reserved;
  unsigned8           clock_seq_low;
  byte                node[6];
} UUID_t;

typedef struct _unsigned64_t {
  unsigned32          lo;
  unsigned32          hi;
} unsigned64_t;

// Forward declarations.
void uuid_create(UUID_t *uuid);
static unsigned16 true_random(void);
void uuid_init(void);

/*----------------------------------------------------------------------------
	Unreal API.
----------------------------------------------------------------------------*/

//
// appGetGUID
//
void appGetGUID( void* GUID )
{
	static int Init = 0;
	
	if( !GUID ) 
		return;
	if( Init == 0 )
	{
		uuid_init();
		Init = 1;
	}
	uuid_create( (UUID_t*)GUID );
}

/*----------------------------------------------------------------------------
	Support functions.
----------------------------------------------------------------------------*/

//
// get_ieee_node_identifier
//
// Required by implementation below.
// Uses 32-bit IP address and 16-bit random number for 48-bit node id.
//
void get_ieee_node_identifier( byte* nodeid )
{
	unsigned32 ip32;
	unsigned16 random16;

	//ip32 = appGetLocalIP();
	ip32 = (((unsigned32) true_random()) << 16) | ((unsigned32) true_random());
	random16 = true_random();

    *(unsigned32*)(&nodeid[0]) = ip32;
    *(unsigned16*)(&nodeid[4]) = random16;
}

/*----------------------------------------------------------------------------
	General implementation.
----------------------------------------------------------------------------*/

/*
**  Add two unsigned 64-bit long integers.
*/
#define ADD_64b_2_64b(A, B, sum) \
  { \
      if (!(((A)->lo & 0x80000000UL) ^ ((B)->lo & 0x80000000UL))) { \
          if (((A)->lo&0x80000000UL)) { \
              (sum)->lo = (A)->lo + (B)->lo; \
              (sum)->hi = (A)->hi + (B)->hi + 1; \
          } \
          else { \
              (sum)->lo  = (A)->lo + (B)->lo; \
              (sum)->hi = (A)->hi + (B)->hi; \
          } \
      } \
      else { \
          (sum)->lo = (A)->lo + (B)->lo; \
          (sum)->hi = (A)->hi + (B)->hi; \
          if (!((sum)->lo&0x80000000UL)) (sum)->hi++; \
      } \
  }

/*
**  Add a 16-bit unsigned integer to a 64-bit unsigned integer.
*/
#define ADD_16b_2_64b(A, B, sum) \
  { \
      (sum)->hi = (B)->hi; \
      if ((B)->lo & 0x80000000UL) { \
          (sum)->lo = (*A) + (B)->lo; \
          if (!((sum)->lo & 0x80000000UL)) (sum)->hi++; \
      } \
      else \
          (sum)->lo = (*A) + (B)->lo; \
  }

/*
**  Global variables.
*/
static unsigned64_t     time_last;
static unsigned16       clock_seq;

static void
mult32(unsigned32 u, unsigned32 v, unsigned64_t *result)
{
  /* Following the notation in Knuth, Vol. 2. */
  unsigned32 uuid1, uuid2, v1, v2, temp;

  uuid1 = u >> 16;
  uuid2 = u & 0xFFFF;
  v1 = v >> 16;
  v2 = v & 0xFFFF;
  temp = uuid2 * v2;
  result->lo = temp & 0xFFFF;
  temp = uuid1 * v2 + (temp >> 16);
  result->hi = temp >> 16;
  temp = uuid2 * v1 + (temp & 0xFFFF);
  result->lo += (temp & 0xFFFF) << 16;
  result->hi += uuid1 * v1 + (temp >> 16);
}

static void
get_system_time(unsigned64_t *uuid_time)
{
  struct timeval tp;
  unsigned64_t utc, usecs, os_basetime_diff;
  gettimeofday(&tp, (struct timezone *)0);
  mult32((long)tp.tv_sec, 10000000, &utc);
  mult32((long)tp.tv_usec, 10, &usecs);
  ADD_64b_2_64b(&usecs, &utc, &utc);

  /* Offset between UUID formatted times and Unix formatted times.
   * UUID UTC base time is October 15, 1582.
   * Unix base time is January 1, 1970. */
  os_basetime_diff.lo = 0x13814000;
  os_basetime_diff.hi = 0x01B21DD2;
  ADD_64b_2_64b(&utc, &os_basetime_diff, uuid_time);
}

/*
** See "The Multiple Prime Random Number Generator" by Alexander
** Hass pp. 368-381, ACM Transactions on Mathematical Software,
** 12/87.
*/
static unsigned32 rand_m;
static unsigned32 rand_ia;
static unsigned32 rand_ib;
static unsigned32 rand_irand;

static void
true_random_init(void)
{
  unsigned64_t t;
  unsigned16 seed;

/* Generating our 'seed' value Start with the current time, but,
* since the resolution of clocks is system hardware dependent and
* most likely coarser than our resolution (10 usec) we 'mixup' the
* bits by xor'ing all the bits together.  This will have the effect
* of involving all of the bits in the determination of the seed
* value while remaining system independent.  Then for good measure
* to ensure a unique seed when there are multiple processes
* creating UUIDs on a system, we add in the PID.
*/
  rand_m = 971;
  rand_ia = 11113;
  rand_ib = 104322;
  rand_irand = 4181;
  get_system_time(&t);
  seed  =  t.lo        & 0xFFFF;
  seed ^= (t.lo >> 16) & 0xFFFF;
  seed ^=  t.hi        & 0xFFFF;
  seed ^= (t.hi >> 16) & 0xFFFF;
  rand_irand += seed + getpid();
}

static unsigned16
true_random(void)
{
  if ((rand_m += 7) >= 9973)
      rand_m -= 9871;
  if ((rand_ia += 1907) >= 99991)
      rand_ia -= 89989;
  if ((rand_ib += 73939) >= 224729)
      rand_ib -= 96233;
  rand_irand = (rand_irand * rand_m) + rand_ia + rand_ib;
  return unsigned16((rand_irand >> 16) ^ (rand_irand & RAND_MASK));
}

/*
**  Startup initialization routine for the UUID module.
*/
void
uuid_init(void)
{
  true_random_init();
  get_system_time(&time_last);
#ifdef NONVOLATILE_CLOCK
  clock_seq = read_clock();
#else
  clock_seq = true_random();
#endif
}

static int
time_cmp(unsigned64_t *time1, unsigned64_t *time2)
{
  if (time1->hi < time2->hi) return -1;
  if (time1->hi > time2->hi) return 1;
  if (time1->lo < time2->lo) return -1;
  if (time1->lo > time2->lo) return 1;
  return 0;
}

static void new_clock_seq(void)
{
  clock_seq = (clock_seq + 1) % (CLOCK_SEQ_LAST + 1);
  if (clock_seq == 0) clock_seq = 1;
#ifdef NONVOLATILE_CLOCK
  write_clock(clock_seq);
#endif
}

void uuid_create(UUID_t *uuid)
{
  static unsigned64_t     time_now;
  static unsigned16       time_adjust;
  byte                    eaddr[6];
  int                     got_no_time = 0;

  get_ieee_node_identifier(eaddr);      /* TO BE PROVIDED */

  do {
      get_system_time(&time_now);
      switch (time_cmp(&time_now, &time_last)) {
      case -1:
          /* Time went backwards. */
          new_clock_seq();
          time_adjust = 0;
          break;
      case 1:
          time_adjust = 0;
          break;
      default:
          if (time_adjust == 0x7FFF)
              /* We're going too fast for our clock; spin. */
              got_no_time = 1;
          else
              time_adjust++;
          break;
      }
  } while (got_no_time);

  time_last.lo = time_now.lo;
  time_last.hi = time_now.hi;

  if (time_adjust != 0) {
      ADD_16b_2_64b(&time_adjust, &time_now, &time_now);
  }

  /* Construct a uuid with the information we've gathered
   * plus a few constants. */
  uuid->time_low = time_now.lo;
  uuid->time_mid = time_now.hi & 0x0000FFFF;
  uuid->time_hi_and_version = (time_now.hi & 0x0FFF0000) >> 16;
  uuid->time_hi_and_version |= (1 << 12);
  uuid->clock_seq_low = clock_seq & 0xFF;
  uuid->clock_seq_hi_and_reserved = (clock_seq & 0x3F00) >> 8;
  uuid->clock_seq_hi_and_reserved |= 0x80;
  memcpy(uuid->node, &eaddr, sizeof uuid->node);
}

#endif

