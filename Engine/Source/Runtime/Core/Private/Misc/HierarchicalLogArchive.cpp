// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/HierarchicalLogArchive.h"

#include "Misc/StringBuilder.h"

FHierarchicalLogArchive::FHierarchicalLogArchive(FArchive& InInnerArchive)
	: FArchiveProxy(InInnerArchive)
	, Indentation(0)
{
}

void FHierarchicalLogArchive::WriteLine(const FString& InLine, bool bIndent)
{
	TAnsiStringBuilder<512> Builder;

	if (Indentation > 0)
	{
		for (int i = 0; i < Indentation - 1; ++i)
		{
			Builder << " |  ";
		}

		Builder << " |-";

		if (bIndent)
		{
			Builder << "-";
		}
		else
		{
			Builder << " ";
		}
	}
	
	if (bIndent)
	{
		Builder << "[+] ";
	}

	Builder << InLine;
	Builder << LINE_TERMINATOR_ANSI;

	Serialize(Builder.GetData(), Builder.Len() * sizeof(ANSICHAR));
}
