// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/InteractiveProcess.h"
#include "HAL/RunnableThread.h"

#if PLATFORM_WINDOWS
#include "Windows/WindowsHWrapper.h"
#endif

static FORCEINLINE bool CreatePipeWrite(void*& ReadPipe, void*& WritePipe)
{
#if PLATFORM_WINDOWS
	SECURITY_ATTRIBUTES Attr = { sizeof(SECURITY_ATTRIBUTES), NULL, true };

	if (!::CreatePipe(&ReadPipe, &WritePipe, &Attr, 0))
	{
		return false;
	}

	if (!::SetHandleInformation(WritePipe, HANDLE_FLAG_INHERIT, 0))
	{
		return false;
	}

	return true;
#else
	return FPlatformProcess::CreatePipe(ReadPipe, WritePipe, true);
#endif // PLATFORM_WINDOWS
}


DEFINE_LOG_CATEGORY(LogInteractiveProcess);

FInteractiveProcess::FInteractiveProcess(const FString& InURL, const FString& InParams, bool InHidden, bool LongTime)
	: FInteractiveProcess(InURL, InParams, FString(), InHidden, LongTime)
{

}

FInteractiveProcess::FInteractiveProcess(const FString& InURL, const FString& InParams, const FString& InWorkingDir, bool InHidden, bool LongTime)
	: bCanceling(false)
	, bHidden(InHidden)
	, bKillTree(false)
	, URL(InURL)
	, Params(InParams)
	, WorkingDir(InWorkingDir)
	, ReadPipeParent(nullptr)
	, WritePipeParent(nullptr)
	, ReadPipeChild(nullptr)
	, WritePipeChild(nullptr)
	, Thread(nullptr)
	, ReturnCode(0)
	, StartTime(0)
	, EndTime(0)
{
	if(LongTime == true)
	{
		SleepTime = 0.0010f; ///< 10 milliseconds sleep
	}
	else
	{
		SleepTime = 0.f;
	}
}

FInteractiveProcess::~FInteractiveProcess()
{
	if (IsRunning() == true)
	{
		Cancel(false);
		Thread->WaitForCompletion();
		delete Thread;
	}
}

FTimespan FInteractiveProcess::GetDuration() const
{
	if (IsRunning() == true)
	{
		return (FDateTime::UtcNow() - StartTime);
	}

	return (EndTime - StartTime);
}

bool FInteractiveProcess::Launch()
{
	if (IsRunning() == true)
	{
		UE_LOGF(LogInteractiveProcess, Warning, "The process is already running");
		return false;
	}

	// For reading from child process
	if (FPlatformProcess::CreatePipe(ReadPipeParent, WritePipeChild) == false)
	{
		UE_LOGF(LogInteractiveProcess, Error, "Failed to create pipes for parent process");
		return false;
	}

	// For writing to child process
	if (CreatePipeWrite(ReadPipeChild, WritePipeParent) == false)
	{
		UE_LOGF(LogInteractiveProcess, Error, "Failed to create pipes for parent process");
		return false;
	}

	ProcessHandle = FPlatformProcess::CreateProc(*URL, *Params, false, bHidden, bHidden, nullptr, 0, WorkingDir.IsEmpty() ? nullptr : *WorkingDir, WritePipeChild, ReadPipeChild);

	if (ProcessHandle.IsValid() == false)
	{
		UE_LOGF(LogInteractiveProcess, Error, "Failed to create process");
		return false;
	}

	// Creating name for the process
	static uint32 tempInteractiveProcessIndex = 0;
	ThreadName = FString::Printf(TEXT("FInteractiveProcess %d"), tempInteractiveProcessIndex);
	tempInteractiveProcessIndex++;

	Thread = FRunnableThread::Create(this, *ThreadName);
	if (Thread == nullptr)
	{
		UE_LOGF(LogInteractiveProcess, Error, "Failed to create process thread!");
		return false;
	}

	UE_LOGF(LogInteractiveProcess, Log, "Process creation succesfull %ls", *ThreadName);

	return true;
}

void FInteractiveProcess::ProcessOutput(const FString& Output)
{
	TArray<FString> LogLines;

	Output.ParseIntoArray(LogLines, TEXT("\n"), false);

	for (int32 LogIndex = 0; LogIndex < LogLines.Num(); ++LogIndex)
	{
		// Don't accept if it is just an empty string
		if (LogLines[LogIndex].IsEmpty() == false)
		{
			OutputDelegate.ExecuteIfBound(LogLines[LogIndex]);
			UE_LOGF(LogInteractiveProcess, Log, "Child Process  -> %ls", *LogLines[LogIndex]);
		}
	}
}

void FInteractiveProcess::SendMessageToProcessIf()
{
	// If there is not a message
	if (StringMessagesToProcess.IsEmpty() == true && DataMessagesToProcess.IsEmpty() == true)
	{
		return;
	}

	if (WritePipeParent == nullptr)
	{
		UE_LOGF(LogInteractiveProcess, Warning, "WritePipe is not valid");
		return;
	}

	if (ProcessHandle.IsValid() == false)
	{
		UE_LOGF(LogInteractiveProcess, Warning, "Process handle is not valid");
		return;
	}

	if (!StringMessagesToProcess.IsEmpty())
	{
		// A string for original message and one for written message
		FString WrittenMessage, Message;
		StringMessagesToProcess.Dequeue(Message);

		FPlatformProcess::WritePipe(WritePipeParent, Message, &WrittenMessage);

		UE_LOGF(LogInteractiveProcess, Log, "Parent Process -> Original Message: %ls , Written Message: %ls", *Message, *WrittenMessage);

		if (WrittenMessage.Len() == 0)
		{
			UE_LOGF(LogInteractiveProcess, Error, "Writing message through pipe failed");
			return;
		}
		else if (Message.Len() > WrittenMessage.Len())
		{
			UE_LOGF(LogInteractiveProcess, Error, "Writing some part of the message through pipe failed");
			return;
		}
	} 
	else if (!DataMessagesToProcess.IsEmpty())
	{
		TArray<uint8> DataMessage;
		DataMessagesToProcess.Dequeue(DataMessage);

		const bool bWritten = FPlatformProcess::WritePipe(WritePipeParent, DataMessage.GetData(), DataMessage.Num());

		if (!bWritten)
		{
			UE_LOGF(LogInteractiveProcess, Error, "Writing message through pipe failed");
			return;
		}
	}
}

void FInteractiveProcess::SendWhenReady(const FString &Message)
{
	StringMessagesToProcess.Enqueue(Message);
}

void FInteractiveProcess::SendWhenReady(const TArray<uint8> &Data)
{
	DataMessagesToProcess.Enqueue(Data);
}

// FRunnable interface
uint32 FInteractiveProcess::Run()
{
	// control and interact with the process
	StartTime = FDateTime::UtcNow();
	{
		do
		{
			FPlatformProcess::Sleep(SleepTime);

			// Read pipe and redirect it to ProcessOutput function
			ProcessOutput(FPlatformProcess::ReadPipe(ReadPipeParent));

			// Write to process if there is a message
			SendMessageToProcessIf();

			// If wanted to stop program
			if (bCanceling == true)
			{
				FPlatformProcess::TerminateProc(ProcessHandle, bKillTree);
				CanceledDelegate.ExecuteIfBound();

				UE_LOGF(LogInteractiveProcess, Log, "The process is being canceled");

				return 0;
			}
		} while (FPlatformProcess::IsProcRunning(ProcessHandle) == true);
	}

	// close pipes
	FPlatformProcess::ClosePipe(ReadPipeParent, WritePipeChild);
	ReadPipeParent = WritePipeChild = nullptr;
	FPlatformProcess::ClosePipe(ReadPipeChild, WritePipeParent);
	ReadPipeChild = WritePipeParent = nullptr;

	// get completion status
	if (FPlatformProcess::GetProcReturnCode(ProcessHandle, &ReturnCode) == false)
	{
		ReturnCode = -1;
	}

	EndTime = FDateTime::UtcNow();

	CompletedDelegate.ExecuteIfBound(ReturnCode, bCanceling);

	return 0;
}
