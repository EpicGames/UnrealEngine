// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/LargeWorldRenderPosition.h"

#include "HAL/Platform.h"
#include "Math/Matrix.inl"
#include "Math/TranslationMatrix.h"
#include "Misc/AssertionMacros.h"
#include "HAL/IConsoleManager.h"

#define UE_LWC_RENDER_TILE_SIZE_MIN (262144.0)	// LWC_TODO: Make this smaller?

//#define UE_LWC_RENDER_TILE_SIZE UE_OLD_WORLD_MAX (FMath::Max(UE_LWC_RENDER_TILE_SIZE_MIN,  HALF_WORLD_MAX / 16777216.0))	// LWC_TODO: Tweak render tile divisor to maximise range.
//static_assert(UE_LWC_RENDER_TILE_SIZE <= 1048576.0, "Current WORLD_MAX is too large and is likely to adversely affect world space coordinate precision within shaders!");	// LWC_TODO: Set this to something reasonable.

//#define UE_LWC_RENDER_TILE_SIZE (262144.0)
//#define UE_LWC_RENDER_TILE_SIZE (1048576.0)
//#define UE_LWC_RENDER_TILE_SIZE (2097152.0)

// This is the max size we allow for LWC offsets relative to the tile
// Value chosen to ensure sufficient precision when stored in single precision float
// Normally offsets should be within +/-TileSizeDivideBy2, but we often rebase multiple quantities off a single tile origin
#define UE_LWC_RENDER_MAX_OFFSET (2097152.0*0.5)

template<typename TScalar>
FVector3f TLargeWorldRenderScalar<TScalar>::GetTileFor(FVector InPosition)
{
	if constexpr (UE_LWC_RENDER_TILE_SIZE == 0)
	{
		return FVector3f::ZeroVector;
	}
	FVector3f LWCTile = FVector3f(InPosition / UE_LWC_RENDER_TILE_SIZE + 0.5);

	// normalize the tile
	LWCTile.X = FMath::FloorToFloat(LWCTile.X);
	LWCTile.Y = FMath::FloorToFloat(LWCTile.Y);
	LWCTile.Z = FMath::FloorToFloat(LWCTile.Z);

	return LWCTile;
}

FMatrix CheckMatrixInTileOffsetRange(const FMatrix& Matrix)
{
	const double OriginMax = UE_LWC_RENDER_MAX_OFFSET;

	const FVector Origin = Matrix.GetOrigin();
	const double OriginX = FMath::Abs(Origin.X);
	const double OriginY = FMath::Abs(Origin.Y);
	const double OriginZ = FMath::Abs(Origin.Z);
	if (OriginX > OriginMax || OriginY > OriginMax || OriginZ > OriginMax)
	{
		ensure(false);
	}
	return Matrix;
}

template<typename TScalar>
FMatrix44f TLargeWorldRenderScalar<TScalar>::SafeCastMatrix(const FMatrix& Matrix)
{
	return FMatrix44f(CheckMatrixInTileOffsetRange(Matrix));
}

template<typename TScalar>
FMatrix44f TLargeWorldRenderScalar<TScalar>::MakeToRelativeWorldMatrix(const FVector Origin, const FMatrix& ToWorld)
{
	return FMatrix44f(MakeToRelativeWorldMatrixDouble(Origin, ToWorld));
}

template<typename TScalar>
FMatrix TLargeWorldRenderScalar<TScalar>::MakeToRelativeWorldMatrixDouble(const FVector Origin, const FMatrix& ToWorld)
{
	return CheckMatrixInTileOffsetRange(ToWorld * FTranslationMatrix(-Origin));
}

template<typename TScalar>
FMatrix44f TLargeWorldRenderScalar<TScalar>::MakeFromRelativeWorldMatrix(const FVector Origin, const FMatrix& FromWorld)
{
	return FMatrix44f(MakeFromRelativeWorldMatrixDouble(Origin, FromWorld));
}

template<typename TScalar>
FMatrix TLargeWorldRenderScalar<TScalar>::MakeFromRelativeWorldMatrixDouble(const FVector Origin, const FMatrix& FromWorld)
{
	return CheckMatrixInTileOffsetRange(FTranslationMatrix(Origin) * FromWorld);
}

template<typename TScalar>
FMatrix44f TLargeWorldRenderScalar<TScalar>::MakeClampedToRelativeWorldMatrix(const FVector Origin, const FMatrix& ToWorld)
{
	return FMatrix44f(MakeClampedToRelativeWorldMatrixDouble(Origin, ToWorld));
}

template<typename TScalar>
FMatrix TLargeWorldRenderScalar<TScalar>::MakeClampedToRelativeWorldMatrixDouble(const FVector Origin, const FMatrix& ToWorld)
{
	const double OriginMax = UE_LWC_RENDER_MAX_OFFSET;

	// Clamp the relative matrix, avoid allowing the relative translation to get too far away from the tile origin
	const FVector RelativeOrigin = ToWorld.GetOrigin() - Origin;
	FVector ClampedRelativeOrigin = RelativeOrigin;
	ClampedRelativeOrigin.X = FMath::Clamp(ClampedRelativeOrigin.X, -OriginMax, OriginMax);
	ClampedRelativeOrigin.Y = FMath::Clamp(ClampedRelativeOrigin.Y, -OriginMax, OriginMax);
	ClampedRelativeOrigin.Z = FMath::Clamp(ClampedRelativeOrigin.Z, -OriginMax, OriginMax);

	FMatrix ClampedToRelativeWorld(ToWorld);
	ClampedToRelativeWorld.SetOrigin(ClampedRelativeOrigin);
	return ClampedToRelativeWorld;
}

// TILE_SIZE = 2097152 = (1 << 21)
//  => Offset < (1 << 21)
// f32 has 23+1 mantissa bits, leaving 3 bits for fractions (24 - 21)
// f64 has 52+1 mantissa bits, leaving 32 bits for fractions (53 - 21)
// This defines a theoretical bound on the error.

template<>
CORE_API void TLargeWorldRenderScalar<float>::Validate(double InAbsolute)
{
	constexpr int FractionBitCount = 3;
	constexpr double Tolerance = 1.0 / (1ULL << FractionBitCount);
	const double CheckAbsolute = GetAbsolute();
	const double Delta = FMath::Abs(CheckAbsolute - InAbsolute);

	ensureMsgf(Delta <= Tolerance, TEXT("Bad TLargeWorldRenderScalar<float> (%.15f) vs (%.15f)"),
		InAbsolute, CheckAbsolute);
}

template<>
CORE_API void TLargeWorldRenderScalar<double>::Validate(double InAbsolute)
{
	constexpr int FractionBitCount = 32;
	constexpr double Tolerance = 1.0 / (1ULL << FractionBitCount);
	const double CheckAbsolute = GetAbsolute();
	const double Delta = FMath::Abs(CheckAbsolute - InAbsolute);

	ensureMsgf(Delta <= Tolerance, TEXT("Bad TLargeWorldRenderScalar<double> (%.15f) vs (%.15f)"),
		InAbsolute, CheckAbsolute);
}

template struct TLargeWorldRenderScalar<double>;
template struct TLargeWorldRenderScalar<float>;

CORE_API int32 GPrimitiveHasTileOffsetData = 1;
static FAutoConsoleVariableRef CVarPrimitiveHasTileOffsetData(
	TEXT("r.PrimitiveHasTileOffsetData"),
	GPrimitiveHasTileOffsetData,
	TEXT("1 to upload lower-precision tileoffset primitive data to gpu, 0 to use higher-precision double float.\n"),
	ECVF_ReadOnly | ECVF_RenderThreadSafe
);
