// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/LocalTimestampDirectoryVisitor.h"

#include "Misc/Paths.h"
#include "Templates/RemoveReference.h"
#include "Templates/UnrealTemplate.h"

/* FLocalTimestampVisitor structors
 *****************************************************************************/

FLocalTimestampDirectoryVisitor::FLocalTimestampDirectoryVisitor( IPlatformFile& InFileInterface, const TArray<FString>& InDirectoriesToIgnore, const TArray<FString>& InDirectoriesToNotRecurse, bool bInCacheDirectories, bool bInMakeLowerCase )
	: bCacheDirectories(bInCacheDirectories)
	, bMakeLowerCase(bInMakeLowerCase)
	, FileInterface(InFileInterface)
{
	// make sure the paths are standardized, since the Visitor will assume they are standard
	for (int32 DirIndex = 0; DirIndex < InDirectoriesToIgnore.Num(); DirIndex++)
	{
		FString DirToIgnore = InDirectoriesToIgnore[DirIndex];
		FPaths::MakeStandardFilename(DirToIgnore);
		DirectoriesToIgnore.Add(DirToIgnore);
	}

	for (int32 DirIndex = 0; DirIndex < InDirectoriesToNotRecurse.Num(); DirIndex++)
	{
		FString DirToNotRecurse = InDirectoriesToNotRecurse[DirIndex];
		FPaths::MakeStandardFilename(DirToNotRecurse);
		DirectoriesToNotRecurse.Add(DirToNotRecurse);
	}
}

FLocalTimestampDirectoryVisitor::~FLocalTimestampDirectoryVisitor() = default;

/* FLocalTimestampVisitor interface
 *****************************************************************************/

bool FLocalTimestampDirectoryVisitor::Visit(const TCHAR* FilenameOrDirectory, bool bIsDirectory)
{
	// make sure all paths are "standardized" so the other end can match up with it's own standardized paths
	FString RelativeFilename = FilenameOrDirectory;
	FPaths::MakeStandardFilename(RelativeFilename);

	if (bMakeLowerCase)
	{
		RelativeFilename.ToLowerInline();
	}

	// cache files and optionally directories
	if (!bIsDirectory)
	{
		FileTimes.Emplace(MoveTemp(RelativeFilename), FileInterface.GetTimeStamp(FilenameOrDirectory));
	}
	else 
	{
		// iterate over directories we care about
		bool bShouldRecurse = true;
		// look in all the ignore directories looking for a match
		for (int32 DirIndex = 0; DirIndex < DirectoriesToIgnore.Num() && bShouldRecurse; DirIndex++)
		{
			if (RelativeFilename.StartsWith(DirectoriesToIgnore[DirIndex]))
			{
				bShouldRecurse = false;
			}
		}

		if (bShouldRecurse == true)
		{
			// If it is a directory that we should not recurse (ie we don't want to process subdirectories of it)
			// handle that case as well...
			for (int32 DirIndex = 0; DirIndex < DirectoriesToNotRecurse.Num() && bShouldRecurse; DirIndex++)
			{
				if (RelativeFilename.StartsWith(DirectoriesToNotRecurse[DirIndex]))
				{
					// Are we more than level deep in that directory?
					FString CheckFilename = RelativeFilename.Right(RelativeFilename.Len() - DirectoriesToNotRecurse[DirIndex].Len());
					if (CheckFilename.Len() > 1)
					{
						bShouldRecurse = false;
					}
				}
			}
		}

		if (bCacheDirectories)
		{
			// we use a timestamp of 0 to indicate a directory
			FileTimes.Emplace(MoveTemp(RelativeFilename), 0);
		}

		// recurse if we should
		if (bShouldRecurse)
		{
			FileInterface.IterateDirectory(FilenameOrDirectory, *this);
		}
	}

	return true;
}
