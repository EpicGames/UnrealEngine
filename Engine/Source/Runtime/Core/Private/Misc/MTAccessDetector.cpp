// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/MTAccessDetector.h"
#include "AutoRTFM.h"

#if ENABLE_MT_DETECTOR

CORE_API bool UE::Private::FMTAccessDetectorOptions::bSuppressCheckFailure = false;

FMRSWRecursiveAccessDetector::FReadersTls& FMRSWRecursiveAccessDetector::GetReadersTls()
{
	thread_local FReadersTls ReadersTls;
	return ReadersTls;
}

FMRSWRecursiveAccessDetector::FDestructionSentinelStackTls& FMRSWRecursiveAccessDetector::GetDestructionSentinelStackTls()
{
	thread_local FDestructionSentinelStackTls DestructionSentinelStackTls;
	return DestructionSentinelStackTls;
}

#endif

#if ENABLE_MT_DETECTOR && WITH_DEV_AUTOMATION_TESTS

#include "Misc/AutomationTest.h"
#include "HAL/Thread.h"
#include "GenericPlatform/GenericPlatformProcess.h"
#include "Tasks/Task.h"
#include "Tests/Benchmark.h"

//----------------------------------------------------------------------//
class UE::Private::FMTAccessDetectorSuppressCheckFailuresForUnitTests
{
public:
	FMTAccessDetectorSuppressCheckFailuresForUnitTests()
		: bOriginalValue(UE::Private::FMTAccessDetectorOptions::bSuppressCheckFailure)
	{
		UE::Private::FMTAccessDetectorOptions::bSuppressCheckFailure = true;
	}

	~FMTAccessDetectorSuppressCheckFailuresForUnitTests()
	{
		UE::Private::FMTAccessDetectorOptions::bSuppressCheckFailure = bOriginalValue;
	}

private:
	bool bOriginalValue = false;
};

//----------------------------------------------------------------------//
// FRWAccessDetector tests
//----------------------------------------------------------------------//
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FRWAccessDetector_ConcurentReadTest, "System.Core.Misc.MTAccessDetector.ConcurrentReadAccess", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
bool FRWAccessDetector_ConcurentReadTest::RunTest(const FString& Parameters)
{
	bool bReading1 = false;
	bool bReading2 = false;
	UE_MT_DECLARE_RW_ACCESS_DETECTOR(MTAccessDetector);

	bool bSuccess = true;
	FThread AccessThread = FThread(TEXT("AccessThread"), [&]()
	{
		bSuccess &= TestTrue(TEXT("Acquiring Read Access 1"), MTAccessDetector.AcquireReadAccess());
		bReading1 = true;
		while (!bReading2) { FPlatformProcess::SleepNoStats(0.01f); }
		bSuccess &= TestTrue(TEXT("Releasing Read Access 1"), MTAccessDetector.ReleaseReadAccess());
		return true;
	});
	while (!bReading1) { FPlatformProcess::SleepNoStats(0.01f); }
	bSuccess &= TestTrue(TEXT("Acquiring Read Access 2"), MTAccessDetector.AcquireReadAccess());
	bReading2 = true;
	AccessThread.Join();
	bSuccess &= TestTrue(TEXT("Releasing Read Access 2"), MTAccessDetector.ReleaseReadAccess());

	return bSuccess;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FRWAccessDetector_ConcurentWriteTest, "System.Core.Misc.MTAccessDetector.ConcurrentWriteAccess", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
bool FRWAccessDetector_ConcurentWriteTest::RunTest(const FString& Parameters)
{
	UE::Private::FMTAccessDetectorSuppressCheckFailuresForUnitTests SuppressCheckFailures;

	bool bWriting1 = false;
	bool bWriting2 = false;
	UE_MT_DECLARE_RW_ACCESS_DETECTOR(MTAccessDetector);

	bool bSuccess = true;
	FThread AccessThread = FThread(TEXT("AccessThread"), [&]()
	{
		bSuccess &= TestTrue(TEXT("Acquiring Write Access"), MTAccessDetector.AcquireWriteAccess());
		bWriting1 = true;
		while (!bWriting2) { FPlatformProcess::SleepNoStats(0.01f); }
		bSuccess &= TestFalse(TEXT("Releasing Read Access"), MTAccessDetector.ReleaseWriteAccess());
		return true;
	});
	while (!bWriting1) { FPlatformProcess::SleepNoStats(0.01f); }
	bSuccess &= TestFalse(TEXT("Acquiring Write Access"), MTAccessDetector.AcquireWriteAccess());
	bWriting2 = true;
	AccessThread.Join();
	bSuccess &= TestTrue(TEXT("Releasing Write Access"), MTAccessDetector.ReleaseWriteAccess());

	return bSuccess;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FRWAccessDetector_ConcurentReadWriteTest, "System.Core.Misc.MTAccessDetector.ConcurrentReadWriteAccess", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
bool FRWAccessDetector_ConcurentReadWriteTest::RunTest(const FString& Parameters)
{
	UE::Private::FMTAccessDetectorSuppressCheckFailuresForUnitTests SuppressCheckFailures;

	bool bReading = false;
	bool bWriting = false;
	UE_MT_DECLARE_RW_ACCESS_DETECTOR(MTAccessDetector);

	bool bSuccess = true;
	FThread AccessThread = FThread(TEXT("AccessThread"), [&]()
	{
		bSuccess &= TestTrue(TEXT("Acquiring Read Access"), MTAccessDetector.AcquireReadAccess());
		bReading = true;
		while (!bWriting) { FPlatformProcess::SleepNoStats(0.01f); }
		bSuccess &= TestFalse(TEXT("Releasing Read Access"), MTAccessDetector.ReleaseReadAccess());
		return true;
	});
	while (!bReading) { FPlatformProcess::SleepNoStats(0.01f); }
	bSuccess &= TestFalse(TEXT("Acquiring Write Access"), MTAccessDetector.AcquireWriteAccess());
	bWriting = true;
	AccessThread.Join();
	bSuccess &= TestTrue(TEXT("Releasing Write Access"), MTAccessDetector.ReleaseWriteAccess());

	return bSuccess;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FRWAccessDetector_WriteReentrance, "System.Core.Misc.MTAccessDetector.WriteReentrance", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
bool FRWAccessDetector_WriteReentrance::RunTest(const FString& Parameters)
{
	UE::Private::FMTAccessDetectorSuppressCheckFailuresForUnitTests SuppressCheckFailures;

	UE_MT_DECLARE_RW_ACCESS_DETECTOR(MTAccessDetector);

	bool bSuccess = true;
	bSuccess &= TestTrue(TEXT("Acquiring 1st Write Access"), MTAccessDetector.AcquireWriteAccess());
	bSuccess &= TestFalse(TEXT("Acquiring 2nd Write Access"), MTAccessDetector.AcquireWriteAccess());
	bSuccess &= TestFalse(TEXT("Releasing Write Access"), MTAccessDetector.ReleaseWriteAccess());
	bSuccess &= TestTrue(TEXT("Releasing Write Access"), MTAccessDetector.ReleaseWriteAccess());

	return bSuccess;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FRWAccessDetector_ReadReentrance, "System.Core.Misc.MTAccessDetector.ReadReentrance", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
bool FRWAccessDetector_ReadReentrance::RunTest(const FString& Parameters)
{
	UE::Private::FMTAccessDetectorSuppressCheckFailuresForUnitTests SuppressCheckFailures;

	UE_MT_DECLARE_RW_ACCESS_DETECTOR(MTAccessDetector);

	bool bSuccess = true;
	bSuccess &= TestTrue(TEXT("Acquiring Write Access"), MTAccessDetector.AcquireWriteAccess());
	bSuccess &= TestFalse(TEXT("Acquiring Read Access"), MTAccessDetector.AcquireReadAccess());
	bSuccess &= TestFalse(TEXT("Releasing Read Access"), MTAccessDetector.ReleaseReadAccess());
	bSuccess &= TestTrue(TEXT("Releasing Write Access"), MTAccessDetector.ReleaseWriteAccess());

	return bSuccess;
}

//----------------------------------------------------------------------//
// FRWRecursiveAccessDetector tests
//----------------------------------------------------------------------//
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FRWAccessDetector_RecursiveConcurentReadTest, "System.Core.Misc.MTAccessDetector.RecursiveConcurrentReadAccess", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
bool FRWAccessDetector_RecursiveConcurentReadTest::RunTest(const FString& Parameters)
{
	bool bReading1 = false;
	bool bReading2 = false;
	UE_MT_DECLARE_RW_RECURSIVE_ACCESS_DETECTOR(MTAccessDetector);

	bool bSuccess = true;
	FThread AccessThread = FThread(TEXT("AccessThread"), [&]()
	{
		bSuccess &= TestTrue(TEXT("Acquiring Read Access 1"), MTAccessDetector.AcquireReadAccess());
		bReading1 = true;
		while (!bReading2) { FPlatformProcess::SleepNoStats(0.01f); }
		bSuccess &= TestTrue(TEXT("Releasing Read Access 1"), MTAccessDetector.ReleaseReadAccess());
		return true;
	});
	while (!bReading1) { FPlatformProcess::SleepNoStats(0.01f); }
	bSuccess &= TestTrue(TEXT("Acquiring Read Access 2"), MTAccessDetector.AcquireReadAccess());
	bReading2 = true;
	AccessThread.Join();
	bSuccess &= TestTrue(TEXT("Releasing Read Access 2"), MTAccessDetector.ReleaseReadAccess());

	return bSuccess;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FRWAccessDetector_RecursiveConcurentWriteTest, "System.Core.Misc.MTAccessDetector.RecursiveConcurrentWriteAccess", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
bool FRWAccessDetector_RecursiveConcurentWriteTest::RunTest(const FString& Parameters)
{
	UE::Private::FMTAccessDetectorSuppressCheckFailuresForUnitTests SuppressCheckFailures;

	bool bWriting1 = false;
	bool bWriting2 = false;
	UE_MT_DECLARE_RW_RECURSIVE_ACCESS_DETECTOR(MTAccessDetector);

	bool bSuccess = true;
	FThread AccessThread = FThread(TEXT("AccessThread"), [&]()
	{
		bSuccess &= TestTrue(TEXT("Acquiring Write Access"), MTAccessDetector.AcquireWriteAccess());
		bWriting1 = true;
		while (!bWriting2) { FPlatformProcess::SleepNoStats(0.01f); }
		bSuccess &= TestFalse(TEXT("Releasing Read Access"), MTAccessDetector.ReleaseWriteAccess());
		return true;
	});
	while (!bWriting1) { FPlatformProcess::SleepNoStats(0.01f); }
	bSuccess &= TestFalse(TEXT("Acquiring Write Access"), MTAccessDetector.AcquireWriteAccess());
	bWriting2 = true;
	AccessThread.Join();
	bSuccess &= TestTrue(TEXT("Releasing Write Access"), MTAccessDetector.ReleaseWriteAccess());

	return bSuccess;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FRWAccessDetector_RecursiveConcurentReadWriteTest, "System.Core.Misc.MTAccessDetector.RecursiveConcurrentReadWriteAccess", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
bool FRWAccessDetector_RecursiveConcurentReadWriteTest::RunTest(const FString& Parameters)
{
	UE::Private::FMTAccessDetectorSuppressCheckFailuresForUnitTests SuppressCheckFailures;

	bool bReading = false;
	bool bWriting = false;
	UE_MT_DECLARE_RW_RECURSIVE_ACCESS_DETECTOR(MTAccessDetector);

	bool bSuccess = true;
	FThread AccessThread = FThread(TEXT("AccessThread"), [&]()
	{
		bSuccess &= TestTrue(TEXT("Acquiring Read Access"), MTAccessDetector.AcquireReadAccess());
		bReading = true;
		while (!bWriting) { FPlatformProcess::SleepNoStats(0.01f); }
		bSuccess &= TestFalse(TEXT("Releasing Read Access"), MTAccessDetector.ReleaseReadAccess());
		return true;
	});
	while (!bReading) { FPlatformProcess::SleepNoStats(0.01f); }
	bSuccess &= TestFalse(TEXT("Acquiring Write Access"), MTAccessDetector.AcquireWriteAccess());
	bWriting = true;
	AccessThread.Join();
	bSuccess &= TestTrue(TEXT("Releasing Write Access"), MTAccessDetector.ReleaseWriteAccess());

	return bSuccess;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FRWAccessDetector_RecursiveWriteReentrance, "System.Core.Misc.MTAccessDetector.RecursiveWriteReentrance", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
bool FRWAccessDetector_RecursiveWriteReentrance::RunTest(const FString& Parameters)
{
	UE_MT_DECLARE_RW_RECURSIVE_ACCESS_DETECTOR(MTAccessDetector);

	bool bSuccess = true;
	bSuccess &= TestTrue(TEXT("Acquiring 1st Write Access"), MTAccessDetector.AcquireWriteAccess());
	bSuccess &= TestTrue(TEXT("Acquiring 2nd Write Access"), MTAccessDetector.AcquireWriteAccess());
	bSuccess &= TestTrue(TEXT("Releasing Write Access"), MTAccessDetector.ReleaseWriteAccess());
	bSuccess &= TestTrue(TEXT("Releasing Write Access"), MTAccessDetector.ReleaseWriteAccess());

	return bSuccess;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FRWAccessDetector_RecursiveReadReentrance, "System.Core.Misc.MTAccessDetector.RecursiveReadReentrance", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
bool FRWAccessDetector_RecursiveReadReentrance::RunTest(const FString& Parameters)
{
	UE::Private::FMTAccessDetectorSuppressCheckFailuresForUnitTests SuppressCheckFailures;

	UE_MT_DECLARE_RW_RECURSIVE_ACCESS_DETECTOR(MTAccessDetector);

	bool bSuccess = true;
	bSuccess &= TestTrue(TEXT("Acquiring Write Access"), MTAccessDetector.AcquireWriteAccess());
	bSuccess &= TestFalse(TEXT("Acquiring Read Access"), MTAccessDetector.AcquireReadAccess());
	bSuccess &= TestFalse(TEXT("Releasing Read Access"), MTAccessDetector.ReleaseReadAccess());
	bSuccess &= TestTrue(TEXT("Releasing Write Access"), MTAccessDetector.ReleaseWriteAccess());

	return bSuccess;
}

//----------------------------------------------------------------------//
// FRWFullyRecursiveAccessDetector tests
//----------------------------------------------------------------------//
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FRWAccessDetector_FullyRecursiveConcurentReadTest, "System.Core.Misc.MTAccessDetector.FullyRecursiveConcurrentReadAccess", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
bool FRWAccessDetector_FullyRecursiveConcurentReadTest::RunTest(const FString& Parameters)
{
	bool bReading1 = false;
	bool bReading2 = false;
	UE_MT_DECLARE_RW_FULLY_RECURSIVE_ACCESS_DETECTOR(MTAccessDetector);

	bool bSuccess = true;
	FThread AccessThread = FThread(TEXT("AccessThread"), [&]()
	{
		bSuccess &= TestTrue(TEXT("Acquiring Read Access 1"), MTAccessDetector.AcquireReadAccess());
		bReading1 = true;
		while (!bReading2) { FPlatformProcess::SleepNoStats(0.01f); }
		bSuccess &= TestTrue(TEXT("Releasing Read Access 1"), MTAccessDetector.ReleaseReadAccess());
		return true;
	});
	while (!bReading1) { FPlatformProcess::SleepNoStats(0.01f); }
	bSuccess &= TestTrue(TEXT("Acquiring Read Access 2"), MTAccessDetector.AcquireReadAccess());
	bReading2 = true;
	AccessThread.Join();
	bSuccess &= TestTrue(TEXT("Releasing Read Access 2"), MTAccessDetector.ReleaseReadAccess());

	return bSuccess;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FRWAccessDetector_FullyRecursiveConcurentWriteTest, "System.Core.Misc.MTAccessDetector.FullyRecursiveConcurrentWriteAccess", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
bool FRWAccessDetector_FullyRecursiveConcurentWriteTest::RunTest(const FString& Parameters)
{
	UE::Private::FMTAccessDetectorSuppressCheckFailuresForUnitTests SuppressCheckFailures;

	bool bWriting1 = false;
	bool bWriting2 = false;
	UE_MT_DECLARE_RW_FULLY_RECURSIVE_ACCESS_DETECTOR(MTAccessDetector);

	bool bSuccess = true;
	FThread AccessThread = FThread(TEXT("AccessThread"), [&]()
	{
		bSuccess &= TestTrue(TEXT("Acquiring Write Access"), MTAccessDetector.AcquireWriteAccess());
		bWriting1 = true;
		while (!bWriting2) { FPlatformProcess::SleepNoStats(0.01f); }
		bSuccess &= TestFalse(TEXT("Releasing Read Access"), MTAccessDetector.ReleaseWriteAccess());
		return true;
	});
	while (!bWriting1) { FPlatformProcess::SleepNoStats(0.01f); }
	bSuccess &= TestFalse(TEXT("Acquiring Write Access"), MTAccessDetector.AcquireWriteAccess());
	bWriting2 = true;
	AccessThread.Join();
	bSuccess &= TestTrue(TEXT("Releasing Write Access"), MTAccessDetector.ReleaseWriteAccess());

	return bSuccess;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FRWAccessDetector_FullyRecursiveConcurentReadWriteTest, "System.Core.Misc.MTAccessDetector.FullyRecursiveConcurrentReadWriteAccess", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
bool FRWAccessDetector_FullyRecursiveConcurentReadWriteTest::RunTest(const FString& Parameters)
{
	UE::Private::FMTAccessDetectorSuppressCheckFailuresForUnitTests SuppressCheckFailures;

	bool bReading = false;
	bool bWriting = false;
	UE_MT_DECLARE_RW_FULLY_RECURSIVE_ACCESS_DETECTOR(MTAccessDetector);

	bool bSuccess = true;
	FThread AccessThread = FThread(TEXT("AccessThread"), [&]()
	{
		bSuccess &= TestTrue(TEXT("Acquiring Read Access"), MTAccessDetector.AcquireReadAccess());
		bReading = true;
		while (!bWriting) { FPlatformProcess::SleepNoStats(0.01f); }
		bSuccess &= TestFalse(TEXT("Releasing Read Access"), MTAccessDetector.ReleaseReadAccess());
		return true;
	});
	while (!bReading) { FPlatformProcess::SleepNoStats(0.01f); }
	bSuccess &= TestFalse(TEXT("Acquiring Write Access"), MTAccessDetector.AcquireWriteAccess());
	bWriting = true;
	AccessThread.Join();
	bSuccess &= TestTrue(TEXT("Releasing Write Access"), MTAccessDetector.ReleaseWriteAccess());

	return bSuccess;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FRWAccessDetector_FullyRecursiveWriteReentrance, "System.Core.Misc.MTAccessDetector.FullyRecursiveWriteReentrance", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
bool FRWAccessDetector_FullyRecursiveWriteReentrance::RunTest(const FString& Parameters)
{
	UE_MT_DECLARE_RW_FULLY_RECURSIVE_ACCESS_DETECTOR(MTAccessDetector);

	bool bSuccess = true;
	bSuccess &= TestTrue(TEXT("Acquiring 1st Write Access"), MTAccessDetector.AcquireWriteAccess());
	bSuccess &= TestTrue(TEXT("Acquiring 2nd Write Access"), MTAccessDetector.AcquireWriteAccess());
	bSuccess &= TestTrue(TEXT("Releasing Write Access"), MTAccessDetector.ReleaseWriteAccess());
	bSuccess &= TestTrue(TEXT("Releasing Write Access"), MTAccessDetector.ReleaseWriteAccess());

	return bSuccess;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FRWAccessDetector_FullyRecursiveReadReentrance, "System.Core.Misc.MTAccessDetector.FullyRecursiveReadReentrance", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
bool FRWAccessDetector_FullyRecursiveReadReentrance::RunTest(const FString& Parameters)
{
	UE_MT_DECLARE_RW_FULLY_RECURSIVE_ACCESS_DETECTOR(MTAccessDetector);

	bool bSuccess = true;
	bSuccess &= TestTrue(TEXT("Acquiring Write Access"), MTAccessDetector.AcquireWriteAccess());
	bSuccess &= TestTrue(TEXT("Acquiring Read Access"), MTAccessDetector.AcquireReadAccess());
	bSuccess &= TestTrue(TEXT("Releasing Read Access"), MTAccessDetector.ReleaseReadAccess());
	bSuccess &= TestTrue(TEXT("Releasing Write Access"), MTAccessDetector.ReleaseWriteAccess());

	return bSuccess;
}

/////////////////////////////////////////////////////////////////////////

namespace MTAccessDetector_Private
{
	using namespace UE::Tasks;

	// helper for testing concurrent cases
	class AUTORTFM_INFER FParallel
	{
	private:
		FTask Task;
		FTaskEvent TaskStarted{ UE_SOURCE_LOCATION };
		FTaskEvent FinishTask{ UE_SOURCE_LOCATION };

	public:
		template<typename PrologType, typename EpilogType>
		FParallel(PrologType&& Prolog, EpilogType&& Epilog)
		{
			Task.Launch(UE_SOURCE_LOCATION, [this, Prolog = MoveTemp(Prolog), Epilog = MoveTemp(Epilog)]
			{
				Prolog();

				TaskStarted.Trigger();
				FinishTask.Wait();

				Epilog();
			});

			TaskStarted.Wait();
		}

		~FParallel()
		{
			FinishTask.Trigger();
			Task.Wait();
		}
	};
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMRSWRecursiveAccessDetectorSingleThreadTest, "System.Core.MRSWRecursiveAccessDetector.SingleThread", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter);

bool FMRSWRecursiveAccessDetectorSingleThreadTest::RunTest(const FString& Parameters)
{
#pragma warning(push)
#pragma warning(disable:6001) // "Using uninitialized memory" warning

	{	// single instance
		UE_MT_DECLARE_MRSW_RECURSIVE_ACCESS_DETECTOR(AD);

		{
			UE_MT_SCOPED_READ_ACCESS(AD);
			UE_MT_SCOPED_READ_ACCESS(AD);
			UE_MT_SCOPED_WRITE_ACCESS(AD);
			UE_MT_SCOPED_WRITE_ACCESS(AD);
			UE_MT_SCOPED_READ_ACCESS(AD);
		}
	}

	{	// multiple nested instances
		UE_MT_DECLARE_MRSW_RECURSIVE_ACCESS_DETECTOR(Outer);
		UE_MT_DECLARE_MRSW_RECURSIVE_ACCESS_DETECTOR(Inner);

		UE_MT_SCOPED_READ_ACCESS(Outer);
		UE_MT_SCOPED_READ_ACCESS(Inner);
		UE_MT_SCOPED_WRITE_ACCESS(Outer);
		UE_MT_SCOPED_WRITE_ACCESS(Inner);
		UE_MT_SCOPED_WRITE_ACCESS(Outer);
		UE_MT_SCOPED_WRITE_ACCESS(Inner);
	}

	{	// destroying detector from inside access scope
		auto* AD = new FMRSWRecursiveAccessDetector;

		FMRSWRecursiveAccessDetector::FDestructionSentinel DestructionSentinel{ FMRSWRecursiveAccessDetector::EAccessType::Reader };
		AD->AcquireReadAccess(DestructionSentinel);

		delete AD;

		if (!DestructionSentinel.bDestroyed)
		{
			AD->ReleaseReadAccess(DestructionSentinel);  //-V774
		}
	}

	return true;

#pragma warning(pop)
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMRSWRecursiveAccessDetectorMultiThreadTest, "System.Core.MRSWRecursiveAccessDetector.MultiThread", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter);

bool FMRSWRecursiveAccessDetectorMultiThreadTest::RunTest(const FString& Parameters)
{
	bool bSuccess = true;

	{	// multireader, allowed
		FMRSWRecursiveAccessDetector AD;
		MTAccessDetector_Private::FParallel ParallelReaderScope([&AD] { AD.AcquireReadAccess(); }, [&AD] { AD.ReleaseReadAccess(); });
		bSuccess &= TestTrueExpr(AD.AcquireReadAccess());
		bSuccess &= TestTrueExpr(AD.ReleaseReadAccess());
	}

	{	// reader/writer, disallowed
		UE::Private::FMTAccessDetectorSuppressCheckFailuresForUnitTests SuppressCheckFailures;

		FMRSWRecursiveAccessDetector AD;
		MTAccessDetector_Private::FParallel ParallelReaderScope([&AD] { AD.AcquireReadAccess(); }, [&AD] { AD.ReleaseReadAccess(); });
		bSuccess &= TestFalseExpr(AD.AcquireWriteAccess()); // must assert on acquiring write access
		bSuccess &= TestTrueExpr(AD.ReleaseWriteAccess());
	}

	{	// multiwriter, disallowed
		UE::Private::FMTAccessDetectorSuppressCheckFailuresForUnitTests SuppressCheckFailures;

		FMRSWRecursiveAccessDetector AD;
		MTAccessDetector_Private::FParallel ParallelReaderScope([&AD] { AD.AcquireWriteAccess(); }, [&AD] { AD.ReleaseWriteAccess(); });
		bSuccess &= TestFalseExpr(AD.AcquireWriteAccess()); // must assert on acquiring write access
		bSuccess &= TestTrueExpr(AD.ReleaseWriteAccess());
	}

	return bSuccess;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMRSWRecursiveAccessDetectorTrivialRelocationTest, "System.Core.MRSWRecursiveAccessDetector.TrivialRelocation", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter);

bool FMRSWRecursiveAccessDetectorTrivialRelocationTest::RunTest(const FString& Parameters)
{
#pragma warning(push)
#pragma warning(disable:6001) // "Using uninitialized memory" warning

#if 0
	{	// trivial relocation by shrinking a TArray while read-accessed is not supported by the race detector. asserts, must provide a useful message
		TArray<FMRSWRecursiveAccessDetector> Arr;

		Arr.AddDefaulted();
		Arr.AddDefaulted();

		Arr[1].AcquireReadAccess(); // acquire read access on the old instance

		Arr.RemoveAtSwap(0); // remove the unused first element

		Arr.Shrink(); // trigger trivial relocation

		Arr[0].ReleaseReadAccess(); // release read access on the new broken instance
	}
#endif

	{	// trivial relocation by shrinking a TArray while write-accessed is supported by the race detector
		TArray<FMRSWRecursiveAccessDetector> Arr;

		Arr.AddDefaulted();
		Arr.AddDefaulted();

		Arr[1].AcquireWriteAccess(); // acquire access on the old instance

		Arr.RemoveAtSwap(0); // remove the unused first element

		Arr.Shrink(); // trigger trivial relocation

		Arr[0].ReleaseWriteAccess(); // release access on the new broken instance
	}

	return true;

#pragma warning(pop)
}

template<uint64 Num, typename AccessDetectorType>
void RWAccessDetectorPerfTest()
{
	AccessDetectorType AD;
	for (int i = 0; i != Num; ++i)
	{
		AD.AcquireReadAccess();
		AD.ReleaseReadAccess();
	}
	for (int i = 0; i != Num; ++i)
	{
		AD.AcquireWriteAccess();
		AD.ReleaseWriteAccess();
	}
}

template<uint64 Num, typename AccessDetectorType>
void RWAccessDetectorConcurrentReadersPerfTest()
{
	using namespace UE::Tasks;

	AccessDetectorType AD;
	auto Reader = [&AD]
	{
		for (uint64 i = 0; i != Num; ++i)
		{
			AD.AcquireReadAccess();
			AD.ReleaseReadAccess();
		}
	};

	TArray<FTask> Tasks;
	for (int i = 0; i != 3; ++i)
	{
		auto TaskBody = Reader;
		Tasks.Add(Launch(UE_SOURCE_LOCATION, MoveTemp(TaskBody)));
	}

	Reader();

	Wait(Tasks);
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FRWRecursiveAccessDetectorPerfTest, "System.Core.RWRecursiveAccessDetector.Perf", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter);

bool FRWRecursiveAccessDetectorPerfTest::RunTest(const FString& Parameters)
{
	UE_BENCHMARK(5, RWAccessDetectorPerfTest<1'000'000, FRWAccessDetector>);
	UE_BENCHMARK(5, RWAccessDetectorPerfTest<1'000'000, FRWRecursiveAccessDetector>);
	UE_BENCHMARK(5, RWAccessDetectorPerfTest<1'000'000, FRWFullyRecursiveAccessDetector>);
	UE_BENCHMARK(5, RWAccessDetectorPerfTest<1'000'000, FMRSWRecursiveAccessDetector>);

	UE_BENCHMARK(5, RWAccessDetectorConcurrentReadersPerfTest<1'000'000, FRWAccessDetector>);
	UE_BENCHMARK(5, RWAccessDetectorConcurrentReadersPerfTest<1'000'000, FRWRecursiveAccessDetector>);
	UE_BENCHMARK(5, RWAccessDetectorConcurrentReadersPerfTest<1'000'000, FRWFullyRecursiveAccessDetector>);
	UE_BENCHMARK(5, RWAccessDetectorConcurrentReadersPerfTest<1'000'000, FMRSWRecursiveAccessDetector>);

	return true;
}

#endif // ENABLE_MT_DETECTOR && WITH_DEV_AUTOMATION_TESTS
