// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/MapErrors.h"

#include "CoreTypes.h"

FLazyName FMapErrors::MatchingActorGUID(TEXT("MatchingActorGUID"));
FLazyName FMapErrors::MatchingLightGUID(TEXT("MatchingLightGUID"));
FLazyName FMapErrors::ActorLargeShadowCaster(TEXT("ActorLargeShadowCaster"));
FLazyName FMapErrors::NoDamageType(TEXT("NoDamageType"));
FLazyName FMapErrors::NonCoPlanarPolys(TEXT("NonCoPlanarPolys"));
FLazyName FMapErrors::SameLocation(TEXT("SameLocation"));
FLazyName FMapErrors::InvalidDrawscale(TEXT("InvalidDrawscale"));
FLazyName FMapErrors::ActorIsObselete(TEXT("ActorIsObselete"));
FLazyName FMapErrors::StaticPhysNone(TEXT("StaticPhysNone"));
FLazyName FMapErrors::VolumeActorCollisionComponentNULL(TEXT("VolumeActorCollisionComponentNULL"));
FLazyName FMapErrors::VolumeActorZeroRadius(TEXT("VolumeActorZeroRadius"));
FLazyName FMapErrors::VertexColorsNotMatchOriginalMesh(TEXT("VertexColorsNotMatchOriginalMesh"));
FLazyName FMapErrors::CollisionEnabledNoCollisionGeom(TEXT("CollisionEnabledNoCollisionGeom"));
FLazyName FMapErrors::ShadowCasterUsingBoundsScale(TEXT("ShadowCasterUsingBoundsScale"));
FLazyName FMapErrors::MultipleSkyLights(TEXT("MultipleSkyLights"));
FLazyName FMapErrors::MultipleSkyAtmospheres(TEXT("MultipleSkyAtmospheres"));
FLazyName FMapErrors::InvalidHairStrandsMaterial(TEXT("InvalidHairStrandsMaterial"));
FLazyName FMapErrors::MultipleSkyAtmosphereTypes(TEXT("MultipleSkyAtmosphereTypes"));
FLazyName FMapErrors::InvalidTrace(TEXT("InvalidTrace"));
FLazyName FMapErrors::BrushZeroPolygons(TEXT("BrushZeroPolygons"));
FLazyName FMapErrors::BrushNullPolygons(TEXT("BrushNullPolygons"));
FLazyName FMapErrors::CleanBSPMaterials(TEXT("CleanBSPMaterials"));
FLazyName FMapErrors::BrushComponentNull(TEXT("BrushComponentNull"));
FLazyName FMapErrors::PlanarBrush(TEXT("PlanarBrush"));
FLazyName FMapErrors::CameraAspectRatioIsZero(TEXT("CameraAspectRatioIsZero"));
FLazyName FMapErrors::AbstractClass(TEXT("AbstractClass"));
FLazyName FMapErrors::DeprecatedClass(TEXT("DeprecatedClass"));
FLazyName FMapErrors::FoliageMissingStaticMesh(TEXT("FoliageMissingStaticMesh"));
FLazyName FMapErrors::FoliageMissingClusterComponent(TEXT("FoliageMissingStaticMesh"));
FLazyName FMapErrors::FixedUpDeletedLayerWeightmap(TEXT("FixedUpDeletedLayerWeightmap"));
FLazyName FMapErrors::FixedUpNonMatchingTargetLayer(TEXT("FixedUpNonMatchingTargetLayer"));
FLazyName FMapErrors::FixedUpInvalidLandscapeMaterialInstances(TEXT("FixedUpInvalidLandscapeMaterialInstances"));
FLazyName FMapErrors::FixedUpIncorrectLayerWeightmap(TEXT("FixedUpIncorrectLayerWeightmap"));
FLazyName FMapErrors::FixedUpSharedLayerWeightmap(TEXT("FixedUpSharedLayerWeightmap"));
FLazyName FMapErrors::LandscapeComponentPostLoad_Warning(TEXT("LandscapeComponentPostLoad_Warning"));
FLazyName FMapErrors::LandscapeLazyObjectPtrDeprecation_Warning(TEXT("LandscapeLazyObjectPtrDeprecation_Warning"));
FLazyName FMapErrors::LandscapeProxyFixupSharedData_Warning(TEXT("LandscapeProxyFixupSharedData_Warning"));
FLazyName FMapErrors::LandscapeProxyInvalidPropertyOverride_Warning(TEXT("LandscapeProxyInvalidPropertyOverride_Warning"));
FLazyName FMapErrors::LandscapeProxyInvalidGrassTypesOverride_Warning(TEXT("LandscapeProxyInvalidGrassTypesOverride_Warning"));
FLazyName FMapErrors::DuplicateLevelInfo(TEXT("DuplicateLevelInfo"));
FLazyName FMapErrors::NoKillZ(TEXT("NoKillZ"));
FLazyName FMapErrors::LightComponentNull(TEXT("LightComponentNull"));
FLazyName FMapErrors::RebuildLighting(TEXT("RebuildLighting"));
FLazyName FMapErrors::StaticComponentHasInvalidLightmapSettings(TEXT("StaticComponentHasInvalidLightmapSettings"));
FLazyName FMapErrors::RebuildPaths(TEXT("RebuildPaths"));
FLazyName FMapErrors::ParticleSystemComponentNull(TEXT("ParticleSystemComponentNull"));
FLazyName FMapErrors::PSysCompErrorEmptyActorRef(TEXT("PSysCompErrorEmptyActorRef"));
FLazyName FMapErrors::PSysCompErrorEmptyMaterialRef(TEXT("PSysCompErrorEmptyMaterialRef"));
FLazyName FMapErrors::SkelMeshActorNoPhysAsset(TEXT("SkelMeshActorNoPhysAsset"));
FLazyName FMapErrors::SkeletalMeshComponent(TEXT("SkeletalMeshComponent"));
FLazyName FMapErrors::SkeletalMeshNull(TEXT("SkeletalMeshNull"));
FLazyName FMapErrors::AudioComponentNull(TEXT("AudioComponentNull"));
FLazyName FMapErrors::SoundCueNull(TEXT("SoundCueNull"));
FLazyName FMapErrors::StaticMeshNull(TEXT("StaticMeshNull"));
FLazyName FMapErrors::StaticMeshComponent(TEXT("StaticMeshComponent"));
FLazyName FMapErrors::SimpleCollisionButNonUniformScale(TEXT("SimpleCollisionButNonUniformScale"));
FLazyName FMapErrors::MoreMaterialsThanReferenced(TEXT("MoreMaterialsThanReferenced"));
FLazyName FMapErrors::MoreNaniteMaterialsThanReferenced(TEXT("MoreNaniteMaterialsThanReferenced"));
FLazyName FMapErrors::ElementsWithZeroTriangles(TEXT("ElementsWithZeroTriangles"));
FLazyName FMapErrors::LevelStreamingVolume(TEXT("LevelStreamingVolume"));
FLazyName FMapErrors::NoLevelsAssociated(TEXT("NoLevelsAssociated"));
FLazyName FMapErrors::FilenameIsTooLongForCooking(TEXT("FilenameIsTooLongForCooking"));
FLazyName FMapErrors::UsingExternalObject(TEXT("UsingExternalObject"));
FLazyName FMapErrors::RepairedPaintedVertexColors(TEXT("RepairedPaintedVertexColors"));
FLazyName FMapErrors::LODActorMissingStaticMesh(TEXT("LODActorMissingStaticMesh"));
FLazyName FMapErrors::LODActorMissingActor(TEXT("LODActorMissingActor"));
FLazyName FMapErrors::LODActorNoActorFound(TEXT("LODActorNoActor"));
FLazyName FMapErrors::InvalidVirtualTextureUsage(TEXT("InvalidVirtualTextureUsage"));
FLazyName FMapErrors::PrimitiveComponentHasInvalidTranslucentShadowSetting(TEXT("PrimitiveComponentHasInvalidTranslucentShadowSetting"));
FLazyName FMapErrors::UseFauxOrthoViewPosDeprecation_Warning(TEXT("UseFauxOrthoViewPosDeprecation_Warning"));
FLazyName FMapErrors::StaticLightWithLumen_Warning(TEXT("StaticLightWithLumen_Warning"));

namespace
{
	const TCHAR* MapErrorsURLPath = TEXT("BuildingWorlds/LevelEditor/MapErrors");
	const TCHAR* MapErrorsLocalPath = TEXT("Shared/Editor/MapErrors");
}

FMapErrorToken::FMapErrorToken(const FName& InErrorName)
	: FDocumentationToken(MapErrorsURLPath, MapErrorsLocalPath, InErrorName.ToString())
{
}

TSharedRef<FMapErrorToken> FMapErrorToken::Create(const FName& InErrorName)
{
	return MakeShareable(new FMapErrorToken(InErrorName));
}
