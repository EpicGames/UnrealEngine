// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/MemStack.h"
#include "Misc/Parse.h"
#include "Misc/CommandLine.h"
#include "Stats/Stats.h"
#include "HAL/IConsoleManager.h"
#include "HAL/LowLevelMemTracker.h"
#include "Misc/CoreDelegates.h"
#include <atomic>

LLM_DEFINE_BOOTSTRAP_TAG(MemStack);

DECLARE_MEMORY_STAT(TEXT("MemStack Large Block"), STAT_MemStackLargeBLock,STATGROUP_Memory);
DECLARE_MEMORY_STAT(TEXT("PageAllocator Free"), STAT_PageAllocatorFree, STATGROUP_Memory);
DECLARE_MEMORY_STAT(TEXT("PageAllocator Used"), STAT_PageAllocatorUsed, STATGROUP_Memory);

FPageAllocator* FPageAllocator::Instance = nullptr;

FPageAllocator::FPageAllocator()
{
	Instance = this;
}

FPageAllocator::~FPageAllocator()
{
	Instance = nullptr;
}

FPageAllocator& FPageAllocator::Get()
{
	if (LIKELY(Instance != nullptr))
	{
		return *Instance;
	}

	return Construct();
}

FORCENOINLINE FPageAllocator& FPageAllocator::Construct()
{
	static FPageAllocator PageAllocator;
	return *Instance;
}

#define USE_MEMSTACK_PURGATORY (0)

#if PLATFORM_WINDOWS && USE_MEMSTACK_PURGATORY

#define MEMSTACK_PURGATORY_COMPILED_IN (1)

#include "Windows/AllowWindowsPlatformTypes.h"
#include <Psapi.h>

TLockFreePointerListUnordered<void, PLATFORM_CACHE_LINE_SIZE> TheAllocatorReady;
TLockFreePointerListUnordered<void, PLATFORM_CACHE_LINE_SIZE> TheSmallAllocatorReady;

#define FRAMES_TO_WAIT (4)
TLockFreePointerListUnordered<void, PLATFORM_CACHE_LINE_SIZE> TheAllocatorPurgatory[FRAMES_TO_WAIT];
TLockFreePointerListUnordered<void, PLATFORM_CACHE_LINE_SIZE> TheSmallAllocatorPurgatory[FRAMES_TO_WAIT];

FThreadSafeCounter BigBlocksOutstanding;
FThreadSafeCounter SmallBlocksOutstanding;

FThreadSafeCounter BigBlocksInUse;
FThreadSafeCounter SmallBlocksInUse;

static int32 GMemStackProtectionTestWrite = 0;
static FAutoConsoleVariableRef CVarTestProtectionWrite(
	TEXT("memstack.TestProtection.Write"),
	GMemStackProtectionTestWrite,
	TEXT("For debugging. If > 0, then a newly freed memstack page will be written to. This will (correctly) cause a GPF the next time a memstack frees a page."));

static int32 GMemStackProtectionTestRead = 0;
static FAutoConsoleVariableRef CVarTestProtectionRead(
	TEXT("memstack.TestProtection.Read"),
	GMemStackProtectionTestRead,
	TEXT("For debugging. If > 0, then a newly freed memstack page will be read from. This will (correctly) cause a GPF the next time a memstack frees a page."));

static int32 GMemStackProtection = 0;
static int32 GMemStackProtectionLatched = 0;

// used to prevent the optimizer from skipping a read.
int32 SomethingWithExternalLinkage = 0;

int32 FrameIndex()
{
	static uint32 LastFrame = MAX_uint32;
	uint32 LocalLastFrame = LastFrame;
	uint32 CurrentFrame = GFrameNumber;
	if (CurrentFrame != LocalLastFrame && LocalLastFrame > FRAMES_TO_WAIT)
	{
		uint32 ToEmpty = (CurrentFrame - (FRAMES_TO_WAIT - 2)) % FRAMES_TO_WAIT; // here we don't empty the one we are about to fill; we do the one before that so we are sure we aren't emptying stuff that is being filled
		while (1)
		{
			void *Block = TheAllocatorPurgatory[ToEmpty].Pop();
			if (!Block)
			{
				break;
			}
			DWORD Old;
			VirtualProtect(Block, FPageAllocator::PageSize, PAGE_READWRITE, &Old);
			check(Old == PAGE_NOACCESS);
			TheAllocatorReady.Push(Block);
		}
		while (1)
		{
			void *Block = TheSmallAllocatorPurgatory[ToEmpty].Pop();
			if (!Block)
			{
				break;
			}
			DWORD Old;
			VirtualProtect(Block, FPageAllocator::SmallPageSize, PAGE_READWRITE, &Old);
			check(Old == PAGE_NOACCESS);
			TheSmallAllocatorReady.Push(Block);
		}
	}
	return int32(CurrentFrame % FRAMES_TO_WAIT);
}

void *FPageAllocator::Alloc(int32 /*Alignment*/)
{
	check(GMemStackProtectionLatched); // attempt to allocate a page before we know if we are doing protected mode or not
	void *Result;
	if (GMemStackProtection)
	{
		Result = TheAllocatorReady.Pop();
		if (!Result)
		{
			Result = VirtualAlloc(NULL, PageSize, MEM_COMMIT | MEM_RESERVE, PAGE_READWRITE);
			BigBlocksOutstanding.Increment();
		}
		BigBlocksInUse.Increment();
	}
	else
	{
		Result = TheAllocator.Allocate();
	}
	STAT(UpdateStats());
	return Result;
}

void FPageAllocator::Free(void *Mem)
{
	if (GMemStackProtection)
	{
		BigBlocksInUse.Decrement();
		DWORD Old;
		verify(VirtualProtect(Mem, PageSize, PAGE_NOACCESS, &Old));
		check(Old == PAGE_READWRITE);
		TheAllocatorPurgatory[FrameIndex()].Push(Mem);

		if (GMemStackProtectionTestWrite)
		{
			*(uint8*)Mem = 0;
		}
		if (GMemStackProtectionTestRead)
		{
			SomethingWithExternalLinkage += *(int32*)Mem;
		}
	}
	else
	{
		TheAllocator.Free(Mem);
	}
	STAT(UpdateStats());
}

void *FPageAllocator::AllocSmall()
{
	check(GMemStackProtectionLatched); // attempt to allocate a page before we know if we are doing protected mode or not
	void *Result;
	if (GMemStackProtection)
	{
		Result = TheSmallAllocatorReady.Pop();
		if (!Result)
		{
			Result = VirtualAlloc(NULL, SmallPageSize, MEM_COMMIT | MEM_RESERVE, PAGE_READWRITE);
			SmallBlocksOutstanding.Increment();
		}
		SmallBlocksInUse.Increment();
	}
	else
	{
		Result = FMemory::Malloc(SmallPageSize);
	}
	STAT(UpdateStats());
	return Result;
}

void FPageAllocator::FreeSmall(void *Mem)
{
	if (GMemStackProtection)
	{
		SmallBlocksInUse.Decrement();
		DWORD Old;
		verify(VirtualProtect(Mem, SmallPageSize, PAGE_NOACCESS, &Old));
		check(Old == PAGE_READWRITE);
		TheSmallAllocatorPurgatory[FrameIndex()].Push(Mem);

		if (GMemStackProtectionTestWrite)
		{
			*(uint8*)Mem = 0;
		}
		if (GMemStackProtectionTestRead)
		{
			SomethingWithExternalLinkage += *(int32*)Mem;
		}
	}
	else
	{
		FMemory::Free(Mem);
	}
	STAT(UpdateStats());
}

uint64 FPageAllocator::BytesUsed()
{
	return uint64(BigBlocksInUse.GetValue()) * PageSize + uint64(SmallBlocksInUse.GetValue()) * SmallPageSize +
		uint64(TheAllocator.GetNumUsed()) * PageSize;
}

uint64 FPageAllocator::BytesFree()
{
	return uint64(BigBlocksOutstanding.GetValue() - BigBlocksInUse.GetValue()) * PageSize + uint64(SmallBlocksOutstanding.GetValue() - SmallBlocksInUse.GetValue()) * SmallPageSize +
		uint64(TheAllocator.GetNumFree()) * PageSize;
}

#include "Windows/HideWindowsPlatformTypes.h"

#else

#define MEMSTACK_PURGATORY_COMPILED_IN (0)

void *FPageAllocator::Alloc(int32 Alignment)
{
#if USING_ADDRESS_SANITISER
	return FMemory::Malloc(FPageAllocator::PageSize, Alignment);
#else
	LLM_SCOPE_BY_BOOTSTRAP_TAG(MemStack);
	void *Result = TheAllocator.Allocate(Alignment);
	STAT(UpdateStats());
	return Result;
#endif
}

void FPageAllocator::Free(void *Mem)
{
#if USING_ADDRESS_SANITISER
	FMemory::Free(Mem);
#else
	TheAllocator.Free(Mem);
	STAT(UpdateStats());
#endif
}

void *FPageAllocator::AllocSmall()
{
	return FMemory::Malloc(SmallPageSize);
}

void FPageAllocator::FreeSmall(void *Mem)
{
	FMemory::Free(Mem);
}

uint64 FPageAllocator::BytesUsed()
{
	return uint64(TheAllocator.GetNumUsed()) * PageSize;
}

uint64 FPageAllocator::BytesFree()
{
	return uint64(TheAllocator.GetNumFree()) * PageSize;
}

#endif

void FPageAllocator::LatchProtectedMode()
{
	FCoreDelegates::GetMemoryTrimDelegate().AddLambda([]()
	{
		Get().TheAllocator.Trim();
	});

#if MEMSTACK_PURGATORY_COMPILED_IN
	const bool bNoProtection = FParse::Param(FCommandLine::Get(), TEXT("NoProtectMemStack"));
	GMemStackProtection = 1;
	if (bNoProtection)
	{
		UE_LOGF(LogTemp, Display, "Enabling purgatory and virtual memory protection to catch stale pointers to mem stacks items (and other users of the page allocator).");
		GMemStackProtection = 0;
	}
	GMemStackProtectionLatched = 1;
#endif
}

#if STATS
void FPageAllocator::UpdateStats()
{
	SET_MEMORY_STAT(STAT_PageAllocatorFree, BytesFree());
	SET_MEMORY_STAT(STAT_PageAllocatorUsed, BytesUsed());
}
#endif


/*-----------------------------------------------------------------------------
	FMemStack implementation.
-----------------------------------------------------------------------------*/
 
FMemStackBase::FMemStackBase(EPageSize InPageSize)
{
	// By fetching the FPageAllocator singleton's address here we can guarantee
	// that its lifetime spans that of all FMemStackBase objects.
	FPageAllocator::Get();
}

int32 FMemStackBase::GetByteCount() const
{
	int32 Count = 0;
	for( FTaggedMemory* Chunk=TopChunk; Chunk; Chunk=Chunk->Next )
	{
		if( Chunk!=TopChunk )
		{
			Count += Chunk->DataSize;
		}
		else
		{
			Count += UE_PTRDIFF_TO_INT32(Top - Chunk->Data());
		}
	}
	return Count;
}

void FMemStackBase::AllocateNewChunk(int32 MinSize)
{
	FTaggedMemory* Chunk=nullptr;
	// Create new chunk.
	int32 TotalSize = MinSize + (int32)sizeof(FTaggedMemory);
	uint32 AllocSize;
	if (TopChunk || TotalSize > FPageAllocator::SmallPageSize || PageSize == EPageSize::Large)
	{
		AllocSize = AlignArbitrary<int32>(TotalSize, FPageAllocator::PageSize);
		if (AllocSize == FPageAllocator::PageSize)
		{
			Chunk = (FTaggedMemory*)FPageAllocator::Get().Alloc();
		}
		else
		{
			Chunk = (FTaggedMemory*)FMemory::Malloc(AllocSize);
			INC_MEMORY_STAT_BY(STAT_MemStackLargeBLock, AllocSize);
		}
		check(AllocSize != FPageAllocator::SmallPageSize);
	}
	else
	{
		AllocSize = FPageAllocator::SmallPageSize;
		Chunk = (FTaggedMemory*)FPageAllocator::Get().AllocSmall();
	}
	Chunk->DataSize = AllocSize - sizeof(FTaggedMemory);

	Chunk->Next = TopChunk;
	TopChunk    = Chunk;
	Top         = Chunk->Data();
	End         = Top + Chunk->DataSize;
}

void FMemStackBase::FreeChunks(FTaggedMemory* NewTopChunk)
{
	while( TopChunk!=NewTopChunk )
	{
		FTaggedMemory* RemoveChunk = TopChunk;
		TopChunk                   = TopChunk->Next;
		if (RemoveChunk->DataSize + sizeof(FTaggedMemory) == FPageAllocator::PageSize)
		{
			FPageAllocator::Get().Free(RemoveChunk);
		}
		else if (RemoveChunk->DataSize + sizeof(FTaggedMemory) == FPageAllocator::SmallPageSize)
		{
			FPageAllocator::Get().FreeSmall(RemoveChunk);
		}
		else
		{
			DEC_MEMORY_STAT_BY(STAT_MemStackLargeBLock, RemoveChunk->DataSize + sizeof(FTaggedMemory));
			FMemory::Free(RemoveChunk);
		}
	}
	Top = nullptr;
	End = nullptr;
	if( TopChunk )
	{
		Top = TopChunk->Data();
		End = Top + TopChunk->DataSize;
	}
}

bool FMemStackBase::ContainsPointer(const void* Pointer) const
{
	const uint8* Ptr = (const uint8*)Pointer;
	for (const FTaggedMemory* Chunk = TopChunk; Chunk; Chunk = Chunk->Next)
	{
		if (Ptr >= Chunk->Data() && Ptr < Chunk->Data() + Chunk->DataSize)
		{
			return true;
		}
	}

	return false;
}

UE_DEFINE_THREAD_SINGLETON_TLS(FMemStack, CORE_API)


FORCENOINLINE void UE::Core::Private::OnInvalidMemStackAllocatorNum(int32 NewNum, SIZE_T NumBytesPerElement)
{
	UE_LOG(LogCore, Fatal, TEXT("Trying to resize TMemStackAllocator to an invalid size of %d with element size %" SIZE_T_FMT), NewNum, NumBytesPerElement);
	for (;;);
}