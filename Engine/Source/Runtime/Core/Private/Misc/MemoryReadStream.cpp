// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/MemoryReadStream.h"

#include "HAL/UnrealMemory.h"
#include "Math/UnrealMathUtility.h"

class FMemoryReadStreamBuffer : public IMemoryReadStream
{
public:
	FMemoryReadStreamBuffer(void* InMemory, int64 InSize, bool InOwnsPointer) : Memory(InMemory), Size(InSize), OwnsPointer(InOwnsPointer) {}

	virtual const void* Read(int64& OutSize, int64 InOffset, int64 InSize) override
	{
		const int64 Offset = FMath::Min(InOffset, Size);
		OutSize = FMath::Min(InSize, Size - Offset);
		return (int8*)Memory + Offset;
	}

	virtual int64 GetSize() override
	{
		return Size;
	}

	virtual ~FMemoryReadStreamBuffer()
	{
		if (OwnsPointer)
		{
			FMemory::Free(Memory);
		}
	}

	void* Memory;
	int64 Size;
	bool OwnsPointer;
};

void IMemoryReadStream::CopyTo(void* Buffer, int64 InOffset, int64 InSize)
{
	int64 ChunkSize = 0;
	int64 ChunkOffset = 0;
	while (ChunkOffset < InSize)
	{
		const void* ChunkMemory = Read(ChunkSize, InOffset + ChunkOffset, InSize - ChunkOffset);
		FMemory::Memcpy((uint8*)Buffer + ChunkOffset, ChunkMemory, ChunkSize);
		ChunkOffset += ChunkSize;
	}
}

IMemoryReadStreamRef IMemoryReadStream::CreateFromCopy(const void* InMemory, int64 InSize)
{
	void* Memory = FMemory::Malloc(InSize);
	FMemory::Memcpy(Memory, InMemory, InSize);
	return new FMemoryReadStreamBuffer(Memory, InSize, true);
}

TRefCountPtr<IMemoryReadStream> IMemoryReadStream::CreateFromCopy(IMemoryReadStream* InStream)
{
	if (InStream)
	{
		const int64 Size = InStream->GetSize();
		void* Memory = FMemory::Malloc(Size);
		InStream->CopyTo(Memory, 0u, Size);
		return new FMemoryReadStreamBuffer(Memory, Size, true);
	}
	return nullptr;
}

TRefCountPtr<IMemoryReadStream> IMemoryReadStream::CreateFromBuffer(void* InMemory, int64 InSize, bool bOwnPointer)
{
	return new FMemoryReadStreamBuffer(InMemory, InSize, bOwnPointer);
}
