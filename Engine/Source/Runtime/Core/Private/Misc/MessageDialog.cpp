// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/MessageDialog.h"

#include "Containers/UnrealString.h"
#include "CoreGlobals.h"
#include "CoreTypes.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Logging/LogVerbosity.h"
#include "Misc/App.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CString.h"
#include "Misc/CoreDelegates.h"
#include "Misc/FeedbackContext.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Trace/Detail/Channel.h"

namespace
{
	/**
	 * Singleton to only create error title text if needed (and after localization system is in place)
	 */
	const FText& GetDefaultMessageTitle()
	{
		// Will be initialised on first call
		static FText DefaultMessageTitle(NSLOCTEXT("MessageDialog", "DefaultMessageTitle", "Message"));
		return DefaultMessageTitle;
	}
}

void FMessageDialog::Debugf( const FText& Message )
{
	Debugf(Message, GetDefaultMessageTitle());
}

void FMessageDialog::Debugf( const FText& Message, const FText& Title )
{
	if( FApp::IsUnattended() == true )
	{
		GLog->Logf( TEXT("%s"), *Message.ToString() );
	}
	else
	{
		if ( GIsEditor && FCoreDelegates::ModalMessageDialog.IsBound() )
		{
			FCoreDelegates::ModalMessageDialog.Execute(EAppMsgCategory::Warning, EAppMsgType::Ok, Message, Title);
		}
		else
		{
			FPlatformMisc::MessageBoxExt( EAppMsgType::Ok, *Message.ToString(), *NSLOCTEXT("MessageDialog", "DefaultDebugMessageTitle", "ShowDebugMessagef").ToString() );
		}
	}
}

void FMessageDialog::ShowLastError()
{
	uint32 LastError = FPlatformMisc::GetLastError();
	TCHAR ErrorBuffer[1024];
	if (FApp::IsUnattended())
	{
		UE_LOGF(LogOutputDevice, Fatal, "GetLastError : %d\n\n%ls", LastError, FPlatformMisc::GetSystemErrorMessage(ErrorBuffer, 1024, LastError));
	}
	else
	{
		TCHAR TempStr[MAX_SPRINTF] = {};
		FCString::Sprintf(TempStr, TEXT("GetLastError : %d\n\n%s"), LastError, FPlatformMisc::GetSystemErrorMessage(ErrorBuffer, 1024, LastError));
		FPlatformMisc::MessageBoxExt(EAppMsgType::Ok, TempStr, *NSLOCTEXT("MessageDialog", "DefaultSystemErrorTitle", "System Error").ToString());
	}
}

EAppReturnType::Type FMessageDialog::Open( EAppMsgType::Type MessageType, const FText& Message )
{
	return Open(EAppMsgCategory::Warning, MessageType, Message, GetDefaultMessageTitle());
}

EAppReturnType::Type FMessageDialog::Open(EAppMsgType::Type MessageType, const FText& Message, const FText& Title)
{
	return Open(EAppMsgCategory::Warning, MessageType, Message, Title);
}

EAppReturnType::Type FMessageDialog::Open( EAppMsgCategory MessageCategory, EAppMsgType::Type MessageType, const FText& Message)
{
	return Open(MessageCategory, MessageType, Message, GetDefaultMessageTitle());
}

EAppReturnType::Type FMessageDialog::Open( EAppMsgCategory MessageCategory, EAppMsgType::Type MessageType, const FText& Message, const FText& Title)
{
	EAppReturnType::Type DefaultValue = EAppReturnType::Yes;
	switch(MessageType)
	{
	case EAppMsgType::Ok:
		DefaultValue = EAppReturnType::Ok;
		break;
	case EAppMsgType::YesNo:
		DefaultValue = EAppReturnType::No;
		break;
	case EAppMsgType::OkCancel:
		DefaultValue = EAppReturnType::Cancel;
		break;
	case EAppMsgType::YesNoCancel:
		DefaultValue = EAppReturnType::Cancel;
		break;
	case EAppMsgType::CancelRetryContinue:
		DefaultValue = EAppReturnType::Cancel;
		break;
	case EAppMsgType::YesNoYesAllNoAll:
		DefaultValue = EAppReturnType::No;
		break;
	case EAppMsgType::YesNoYesAllNoAllCancel:
	default:
		DefaultValue = EAppReturnType::Yes;
		break;
	}

	if (GIsRunningUnattendedScript && MessageType != EAppMsgType::Ok)
	{
		if (GWarn)
		{
			GWarn->Logf(TEXT("Message Dialog was triggered in unattended script mode without a default value. %d will be used."), (int32)DefaultValue);
		}

		if (FPlatformMisc::IsDebuggerPresent())
		{
			UE_DEBUG_BREAK();
		}
		else
		{
			FDebug::DumpStackTraceToLog(ELogVerbosity::Error);
		}
	}

	return Open(MessageCategory, MessageType, DefaultValue, Message, Title);
}

EAppReturnType::Type FMessageDialog::Open(EAppMsgType::Type MessageType, EAppReturnType::Type DefaultValue, const FText& Message)
{
	return Open(EAppMsgCategory::Warning, MessageType, DefaultValue, Message, GetDefaultMessageTitle());
}

EAppReturnType::Type FMessageDialog::Open(EAppMsgType::Type MessageType, EAppReturnType::Type DefaultValue, const FText& Message, const FText& Title)
{
	return Open(EAppMsgCategory::Warning, MessageType, DefaultValue, Message, Title);
}

EAppReturnType::Type FMessageDialog::Open(EAppMsgCategory MessageCategory, EAppMsgType::Type MessageType, EAppReturnType::Type DefaultValue, const FText& Message)
{
	return Open(MessageCategory, MessageType, DefaultValue, Message, GetDefaultMessageTitle());
}

EAppReturnType::Type FMessageDialog::Open(EAppMsgCategory MessageCategory, EAppMsgType::Type MessageType, EAppReturnType::Type DefaultValue, const FText& Message, const FText& Title)
{
	EAppReturnType::Type Result = DefaultValue;

	if (!FApp::IsUnattended() && !GIsRunningUnattendedScript)
	{
		if ( GIsEditor && !IsRunningCommandlet() && FCoreDelegates::ModalMessageDialog.IsBound() )
		{
			Result = FCoreDelegates::ModalMessageDialog.Execute( MessageCategory, MessageType, Message, Title );
		}
		else
		{
			Result = FPlatformMisc::MessageBoxExt( MessageType, *Message.ToString(), *Title.ToString() );
		}
	}

	if (GWarn)
	{
		GWarn->Logf(TEXT("Message dialog closed, result: %s, title: %s, text: %s"), LexToString(Result), *Title.ToString(), *Message.ToString());
	}

	return Result;
}
