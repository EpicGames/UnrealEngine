// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Misc/MessageDialog.h"
#include "HAL/IConsoleManager.h"

#if (WITH_EDITOR || IS_PROGRAM) && !UE_BUILD_SHIPPING

static void TestMessageDialog()
{
	auto ShouldContinueTest = [](EAppReturnType::Type RetVal)
	{
		return RetVal == EAppReturnType::Yes
			|| RetVal == EAppReturnType::YesAll
			|| RetVal == EAppReturnType::Ok
			|| RetVal == EAppReturnType::Retry
			|| RetVal == EAppReturnType::Continue;
	};

	int32 CurrentMsgCategory = 0;
	const EAppMsgCategory MsgCategories[] = {
		EAppMsgCategory::Warning,
		EAppMsgCategory::Error,
		EAppMsgCategory::Success,
		EAppMsgCategory::Info,
	};

	int32 CurrentMsgType = 0;
	const EAppMsgType::Type MsgTypes[] = {
		EAppMsgType::Ok,
		EAppMsgType::YesNo,
		EAppMsgType::OkCancel,
		EAppMsgType::YesNoCancel,
		EAppMsgType::CancelRetryContinue,
		EAppMsgType::YesNoYesAllNoAll,
		EAppMsgType::YesNoYesAllNoAllCancel,
		EAppMsgType::YesNoYesAll,
	};

	FText Message = FText::FromString("This is an important message.");
	FText LongMessage = FText::FromString("This is a really long important message. Lorem ipsum dolor sit amet, consectetur adipiscing elit. Pellentesque nisi lectus, accumsan in ex a, varius ullamcorper turpis. Sed elementum eleifend nisl vitae pellentesque. Ut ac laoreet augue. Interdum et malesuada fames ac ante ipsum primis in faucibus. Nam ut lacus tortor. Fusce ante purus, ultricies sed rhoncus accumsan, egestas sed leo. Nam sed odio diam. Nullam ac ex consequat, viverra ante id, sodales dui. Suspendisse cursus urna vel arcu pulvinar fermentum. Vivamus et maximus nunc, sed dictum nibh. Ut ex nisl, iaculis et convallis ut, viverra a ligula. Donec nisl sapien, consectetur ut porttitor a, laoreet non ante. Quisque.  Lorem ipsum dolor sit amet, consectetur adipiscing elit. Pellentesque nisi lectus, accumsan in ex a, varius ullamcorper turpis. Sed elementum eleifend nisl vitae pellentesque. Ut ac laoreet augue. Interdum et malesuada fames ac ante ipsum primis in faucibus. Nam ut lacus tortor. Fusce ante purus, ultricies sed rhoncus accumsan, egestas sed leo. Nam sed odio diam. Nullam ac ex consequat, viverra ante id, sodales dui. Suspendisse cursus urna vel arcu pulvinar fermentum. Vivamus et maximus nunc, sed dictum nibh. Ut ex nisl, iaculis et convallis ut, viverra a ligula. Donec nisl sapien, consectetur ut porttitor a, laoreet non ante. Quisque.");
	FText DialogTitle = FText::FromString("Message Title");

	while (ShouldContinueTest(FMessageDialog::Open(MsgCategories[CurrentMsgCategory], MsgTypes[CurrentMsgType], Message, DialogTitle)))
	{
		CurrentMsgCategory = (CurrentMsgCategory + 1) % UE_ARRAY_COUNT(MsgCategories);
		CurrentMsgType = (CurrentMsgType + 1) % UE_ARRAY_COUNT(MsgTypes);
	}
}

FAutoConsoleCommand TestMessageDialogCommand(TEXT("Slate.TestMessageDialog"), TEXT(""), FConsoleCommandDelegate::CreateStatic(&TestMessageDialog));

#endif