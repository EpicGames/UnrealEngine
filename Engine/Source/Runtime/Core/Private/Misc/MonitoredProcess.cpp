// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/MonitoredProcess.h"
#include "HAL/RunnableThread.h"
#include "Misc/Paths.h"
#include "Logging/LogMacros.h"
#include "Async/Async.h"
#include "Misc/CoreDelegates.h"
#include <atomic>

DEFINE_LOG_CATEGORY_STATIC(LogMonitoredProcess, Log, All);

/* FMonitoredProcess structors
 *****************************************************************************/

FMonitoredProcess::FMonitoredProcess( const FString& InURL, const FString& InParams, bool InHidden, bool InCreatePipes )
	: FMonitoredProcess(InURL, InParams, FPaths::RootDir(), InHidden, InCreatePipes)
{ 
}

FMonitoredProcess::FMonitoredProcess( const FString& InURL, const FString& InParams, const FString& InWorkingDir, bool InHidden, bool InCreatePipes )
	: Hidden(InHidden)
	, Params(InParams)
	, bIsRunning(false)
	, URL(InURL)
	, WorkingDir(InWorkingDir)
	, bCreatePipes(InCreatePipes)
{ 
}

bool FMonitoredProcess::RunProcessSynchronous( const FString& InURL, const FString& InParams, TArray<FString>& OutOutput, float Timeout/*0.f*/ )
{
	FMonitoredProcess MonitoredProcess(InURL, InParams, /*InHidden=*/true);

	MonitoredProcess.OnOutput().BindLambda([&OutOutput](const FString& Output)
		{
			OutOutput.Add(Output);
		});

	if (!MonitoredProcess.Launch())
	{
		return false;
	}

	double StartTime = FPlatformTime::Seconds();
	while (MonitoredProcess.Update())
	{
		if (Timeout > 0.f)
		{
			if ((FPlatformTime::Seconds() - StartTime) >= Timeout)
			{
				MonitoredProcess.Cancel(true);
				return false;
			}
		}
		FPlatformProcess::Sleep(0.01f);
	}

	// Add any remaining unprocessed output.
	const FString& RemainingBuffer = MonitoredProcess.GetFullOutputWithoutDelegate();
	if (RemainingBuffer.Len() > 0)
	{
		OutOutput.Add(RemainingBuffer);
	}

	return MonitoredProcess.GetReturnCode() == 0;
}

FMonitoredProcess::~FMonitoredProcess()
{
	if (bIsRunning)
	{
		Cancel(true);
	}

	if (Thread != nullptr) 
	{
		Thread->WaitForCompletion();
		delete Thread;
	}
}

/* FMonitoredProcess interface
 *****************************************************************************/

FTimespan FMonitoredProcess::GetDuration() const
{
	if (StartTime.Load() == 0)
	{
		// Not initialized yet - monitoring thread might not have set StartTime yet.
		return 0;
	}
	if (bIsRunning)
	{
		return (FDateTime::UtcNow() - StartTime.Load());
	}

	return (EndTime.Load() - StartTime.Load());
}


bool FMonitoredProcess::Launch()
{
	if (bIsRunning)
	{
		return false;
	}

	check (Thread == nullptr); // We shouldn't be calling this twice

	if (bCreatePipes && !FPlatformProcess::CreatePipe(ReadPipe, WritePipe))
	{
		return false;
	}

	ProcessHandle = FPlatformProcess::CreateProc(*URL, *Params, false, Hidden, Hidden, nullptr, 0, *WorkingDir, WritePipe, nullptr);

	if (!ProcessHandle.IsValid())
	{
		return false;
	}

	static std::atomic<uint32> MonitoredProcessIndex { 0 };
	const FString MonitoredProcessName = FString::Printf( TEXT( "FMonitoredProcess %d" ), MonitoredProcessIndex.fetch_add(1) );

	bIsRunning = true;
	Thread = FRunnableThread::Create(this, *MonitoredProcessName, 128 * 1024, TPri_AboveNormal);
	if ( !FPlatformProcess::SupportsMultithreading() )
	{
		StartTime.Store(FDateTime::UtcNow());
	}

	return true;
}

int FMonitoredProcess::GetReturnCode() const
{
	check(!bIsRunning);
	return ReturnCode;
}

const FString& FMonitoredProcess::GetFullOutputWithoutDelegate() const
{
	check((!FPlatformProcess::SupportsMultithreading()) || (!bIsRunning));	// Don't touch the output buffer while running with a background task
	return OutputBuffer;
}

/* FMonitoredProcess implementation
 *****************************************************************************/

void FMonitoredProcess::ProcessOutput( const FString& Output )
{
	// Append this output to the output buffer
	OutputBuffer += Output;

	// if the delegate is not bound, then just keep buffering the output to OutputBuffer for later return from GetFullOutputWithoutDelegate()
	if (OutputDelegate.IsBound())
	{
		// Output all the complete lines
		int32 LineStartIdx = 0;
		for (int32 Idx = 0; Idx < OutputBuffer.Len(); Idx++)
		{
			if (OutputBuffer[Idx] == '\r' || OutputBuffer[Idx] == '\n')
			{
				OutputDelegate.ExecuteIfBound(OutputBuffer.Mid(LineStartIdx, Idx - LineStartIdx));

				if (OutputBuffer[Idx] == '\r' && Idx + 1 < OutputBuffer.Len() && OutputBuffer[Idx + 1] == '\n')
				{
					Idx++;
				}

				LineStartIdx = Idx + 1;
			}
		}

		// Remove all the complete lines from the buffer
		OutputBuffer.MidInline(LineStartIdx, MAX_int32, EAllowShrinking::No);
	}
}

void FMonitoredProcess::TickInternal()
{
	// monitor the process

	// kill child process if we choose to cancel or it the engine is shutting down
	if (Canceling)
	{
		FPlatformProcess::TerminateProc(ProcessHandle, KillTree);

		EndTime.Store(FDateTime::UtcNow());

		// Make sure to read the final bytes from the pipe before closing
		ProcessOutput(FPlatformProcess::ReadPipe(ReadPipe));

		// close output pipes
		FPlatformProcess::ClosePipe(ReadPipe, WritePipe);
		ReadPipe = WritePipe = nullptr;

		// Indicate failure in the return code
		ReturnCode.Store(-1);

		bIsRunning = false;
		CanceledDelegate.ExecuteIfBound();
	}
	else if (!FPlatformProcess::IsProcRunning(ProcessHandle))
	{
		EndTime.Store(FDateTime::UtcNow());

		// Make sure to read the final bytes from the pipe before closing
		ProcessOutput(FPlatformProcess::ReadPipe(ReadPipe));

		// close output pipes
		FPlatformProcess::ClosePipe(ReadPipe, WritePipe);
		ReadPipe = WritePipe = nullptr;

		int ProcessReturnCode = 0;
		// get completion status
		if (FPlatformProcess::GetProcReturnCode(ProcessHandle, &ProcessReturnCode))
		{
			ReturnCode.Store(ProcessReturnCode);
		}
		else
		{
			ReturnCode.Store(-1);
		}

		bIsRunning = false;
		CompletedDelegate.ExecuteIfBound(ReturnCode);
	}
	else
	{
		ProcessOutput(FPlatformProcess::ReadPipe(ReadPipe));
	}
}


bool FMonitoredProcess::Update()
{
	if (!FPlatformProcess::SupportsMultithreading())
	{
		FPlatformProcess::Sleep(SleepInterval);
		Tick();
	}
	return bIsRunning;
}


/* FRunnable interface
 *****************************************************************************/

uint32 FMonitoredProcess::Run()
{
	StartTime.Store(FDateTime::UtcNow());
	while (bIsRunning)
	{
		FPlatformProcess::Sleep(SleepInterval);
		TickInternal();
	} 

	return 0;
}

/* FRunnableSingleThreaded interface
*****************************************************************************/
void FMonitoredProcess::Tick()
{
	if (bIsRunning)
	{
		TickInternal();
	}
}


/* FSerializedUATProcess
*****************************************************************************/

FCriticalSection FSerializedUATProcess::Serializer;
FSerializedUATProcess* FSerializedUATProcess::HeadProcess = nullptr;
bool FSerializedUATProcess::bHasSucceededOnce = false;

void FSerializedUATProcess::CancelQueue()
{
	FScopeLock Lock(&Serializer);
	if (HeadProcess)
	{
		// delete anything after Head
		FSerializedUATProcess* Travel = HeadProcess->NextProcessToRun;
		while (Travel != nullptr)
		{
			FSerializedUATProcess* Delete = Travel;
			Travel = Travel->NextProcessToRun;
			delete Delete;
		}


		Travel = HeadProcess;
		HeadProcess = nullptr;
		Travel->NextProcessToRun = nullptr;
		Travel->Cancel(true);

		// can/should we block here?
	}
}

FSerializedUATProcess::FSerializedUATProcess(const FString& RunUATCommandline)
	// we will modify URL and Params in this constructor, so there's no need to pass anything up to base
	: FMonitoredProcess("", "", true, true)
{
	// replace the URL and Params freom base with the shelled version

#if PLATFORM_WINDOWS
	URL = TEXT("cmd.exe");
	Params = FString::Printf(TEXT("/c \"\"%s\" %s\""), *GetUATPath(), *RunUATCommandline);
#elif PLATFORM_MAC || PLATFORM_LINUX
	URL = TEXT("/usr/bin/env");
	Params = FString::Printf(TEXT(" -- \"%s\" %s"), *GetUATPath(), *RunUATCommandline);
#endif

	static bool bHasSetupDelegate = false;
	if (!bHasSetupDelegate)
	{
		bHasSetupDelegate = true;
		FCoreDelegates::OnShutdownAfterError.AddStatic(&CancelQueue);
		FCoreDelegates::OnExit.AddStatic(&CancelQueue);
	}
}

bool FSerializedUATProcess::Launch()
{
	FScopeLock Lock(&Serializer);

	// are we first in line?
	if (HeadProcess == nullptr)
	{
		HeadProcess = this;

		return LaunchInternal();
	}
	else
	{
		// put us last in line
		for (FSerializedUATProcess* Travel = HeadProcess; Travel; Travel = Travel->NextProcessToRun)
		{
			if (Travel->NextProcessToRun == nullptr)
			{
				Travel->NextProcessToRun = this;
				break;
			}
		}

		// and return immediately
		return true;
	}
}

bool FSerializedUATProcess::LaunchNext()
{
	FScopeLock Lock(&Serializer);
	if (HeadProcess == nullptr)
	{
		// can happen during shutdown
		return false;
	}
	
	check(this == HeadProcess);

	FSerializedUATProcess* FinishedTask = HeadProcess;
	HeadProcess = HeadProcess->NextProcessToRun;
//	delete FinishedTask;

	if (HeadProcess != nullptr)
	{
		return HeadProcess->LaunchInternal();
	}
	return false;
}

bool FSerializedUATProcess::LaunchInternal()
{
	if (bHasSucceededOnce)
	{
		Params += TEXT(" -nocompile -nocompileuat");
//		Params += TEXT(" -nocompile");
	}

	FOnMonitoredProcessCompleted OriginalCompletedDelegate = CompletedDelegate;
	FSimpleDelegate OriginalCanceledDelegate = CanceledDelegate;

	CompletedDelegate.BindLambda([this, OriginalCompletedDelegate](int32 ExitCode)
		{
			if (ExitCode == 0 || ExitCode == 10)
			{
				bHasSucceededOnce = true;
			}
			OriginalCompletedDelegate.ExecuteIfBound(ExitCode);

			LaunchNext();
		});
	CanceledDelegate.BindLambda([this, OriginalCanceledDelegate]()
		{
			OriginalCanceledDelegate.ExecuteIfBound();

			LaunchNext();
		});

	UE_LOGF(LogMonitoredProcess, Log, "Running Serialized UAT: [ %ls %ls ]", *URL, *Params);

	if (FMonitoredProcess::Launch() == false)
	{
		LaunchFailedDelegate.ExecuteIfBound();

		LaunchNext();
	}

	return true;
}

FString FSerializedUATProcess::GetUATPath()
{
#if PLATFORM_WINDOWS
	FString RunUATScriptName = "RunUAT.bat";
#elif PLATFORM_LINUX
	FString RunUATScriptName = "RunUAT.sh";
#else
	FString RunUATScriptName = "RunUAT.command";
#endif

	return FPaths::ConvertRelativePathToFull(FPaths::EngineDir() / TEXT("Build/BatchFiles") / RunUATScriptName);
}

