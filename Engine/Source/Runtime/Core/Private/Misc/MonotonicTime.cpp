// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/MonotonicTime.h"

#include "HAL/PlatformTime.h"

namespace UE
{

FMonotonicTimePoint FMonotonicTimePoint::Now()
{
	return FromSeconds(FPlatformTime::Seconds());
}

} // namespace UE
