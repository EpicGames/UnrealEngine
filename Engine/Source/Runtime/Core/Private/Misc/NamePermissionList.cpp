// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/NamePermissionList.h"

#include "Templates/Tuple.h"
#include "Templates/UnrealTemplate.h"


bool FNamePermissionList::PassesFilter(const FName Item) const
{
	if (DenyListAll.Num() > 0)
	{
		return false;
	}

	if (AllowList.Num() > 0 && !AllowList.Contains(Item))
	{
		return false;
	}

	if (DenyList.Contains(Item))
	{
		return false;
	}

	return true;
}

bool FNamePermissionList::AddDenyListItem(const FName OwnerName, const FName Item)
{
	const int32 OldNum = DenyList.Num();
	DenyList.FindOrAdd(Item).AddUnique(OwnerName);

	const bool bFilterChanged = OldNum != DenyList.Num();
	if (bFilterChanged && !bSuppressOnFilterChanged)
	{
		OnFilterChanged().Broadcast();
	}

	return bFilterChanged;
}

bool FNamePermissionList::AddAllowListItem(const FName OwnerName, const FName Item)
{
	const int32 OldNum = AllowList.Num();
	AllowList.FindOrAdd(Item).AddUnique(OwnerName);

	const bool bFilterChanged = OldNum != AllowList.Num();
	if (bFilterChanged && !bSuppressOnFilterChanged)
	{
		OnFilterChanged().Broadcast();
	}

	return bFilterChanged;
}

bool FNamePermissionList::RemoveDenyListItem(const FName OwnerName, const FName Item)
{
	FPermissionListOwners* Owners = DenyList.Find(Item);
	if (Owners && Owners->Remove(OwnerName) == 1)
	{
		if (Owners->Num() == 0)
		{
			DenyList.Remove(Item);
		}
		if (!bSuppressOnFilterChanged)
		{
			OnFilterChanged().Broadcast();
		}
		return true;
	}

	return false;
}

bool FNamePermissionList::RemoveAllowListItem(const FName OwnerName, const FName Item)
{
	FPermissionListOwners* Owners = AllowList.Find(Item);
	if (Owners && Owners->Remove(OwnerName) == 1)
	{
		if (Owners->Num() == 0)
		{
			AllowList.Remove(Item);
		}
		if (!bSuppressOnFilterChanged)
		{
			OnFilterChanged().Broadcast();
		}
		return true;
	}

	return false;
}

bool FNamePermissionList::AddDenyListAll(const FName OwnerName)
{
	const int32 OldNum = DenyListAll.Num();
	DenyListAll.AddUnique(OwnerName);

	const bool bFilterChanged = OldNum != DenyListAll.Num();
	if (bFilterChanged && !bSuppressOnFilterChanged)
	{
		OnFilterChanged().Broadcast();
	}

	return bFilterChanged;
}

bool FNamePermissionList::HasFiltering() const
{
	return DenyList.Num() > 0 || AllowList.Num() > 0 || DenyListAll.Num() > 0;
}

TArray<FName> FNamePermissionList::GetOwnerNames() const
{
	TArray<FName> OwnerNames;
	
	for (const auto& It : DenyList)
	{
		for (const auto& OwnerName : It.Value)
		{
			OwnerNames.AddUnique(OwnerName);
		}
	}

	for (const auto& It : AllowList)
	{
		for (const auto& OwnerName : It.Value)
		{
			OwnerNames.AddUnique(OwnerName);
		}
	}

	for (const auto& OwnerName : DenyListAll)
	{
		OwnerNames.AddUnique(OwnerName);
	}

	return OwnerNames;
}

bool FNamePermissionList::UnregisterOwner(const FName OwnerName)
{
	bool bFilterChanged = false;

	for (auto It = DenyList.CreateIterator(); It; ++It)
	{
		It->Value.Remove(OwnerName);
		if (It->Value.Num() == 0)
		{
			It.RemoveCurrent();
			bFilterChanged = true;
		}
	}

	for (auto It = AllowList.CreateIterator(); It; ++It)
	{
		It->Value.Remove(OwnerName);
		if (It->Value.Num() == 0)
		{
			It.RemoveCurrent();
			bFilterChanged = true;
		}
	}

	bFilterChanged |= (DenyListAll.Remove(OwnerName) > 0);

	if (bFilterChanged && !bSuppressOnFilterChanged)
	{
		OnFilterChanged().Broadcast();
	}

	return bFilterChanged;
}

bool FNamePermissionList::UnregisterOwners(const TArray<FName>& OwnerNames)
{
	bool bFilterChanged = false;
	{
		TGuardValue<bool> Guard(bSuppressOnFilterChanged, true);

		for (FName OwnerName : OwnerNames)
		{
			bFilterChanged |= UnregisterOwner(OwnerName);
		}
	}

	if (bFilterChanged && !bSuppressOnFilterChanged)
	{
		OnFilterChanged().Broadcast();
	}

	return bFilterChanged;
}

bool FNamePermissionList::Append(const FNamePermissionList& Other)
{
	bool bFilterChanged = false;
	{
		TGuardValue<bool> Guard(bSuppressOnFilterChanged, true);

		for (const auto& It : Other.DenyList)
		{
			for (const auto& OwnerName : It.Value)
			{
				bFilterChanged |= AddDenyListItem(OwnerName, It.Key);
			}
		}

		for (const auto& It : Other.AllowList)
		{
			for (const auto& OwnerName : It.Value)
			{
				bFilterChanged |= AddAllowListItem(OwnerName, It.Key);
			}
		}

		for (const auto& OwnerName : Other.DenyListAll)
		{
			bFilterChanged |= AddDenyListAll(OwnerName);
		}
	}

	if (bFilterChanged && !bSuppressOnFilterChanged)
	{
		OnFilterChanged().Broadcast();
	}

	return bFilterChanged;
}

bool FNamePermissionList::UnregisterOwnersAndAppend(const TArray<FName>& OwnerNamesToRemove, const FNamePermissionList& FiltersToAdd)
{
	bool bFilterChanged = false;
	{
		TGuardValue<bool> Guard(bSuppressOnFilterChanged, true);

		bFilterChanged |= UnregisterOwners(OwnerNamesToRemove);
		bFilterChanged |= Append(FiltersToAdd);
	}

	if (bFilterChanged && !bSuppressOnFilterChanged)
	{
		OnFilterChanged().Broadcast();
	}

	return bFilterChanged;
}
