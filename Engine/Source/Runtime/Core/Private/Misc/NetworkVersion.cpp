// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/NetworkVersion.h"

#include "BuildSettings.h"
#include "HAL/IConsoleManager.h"
#include "Logging/LogCategory.h"
#include "Misc/App.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CString.h"
#include "Misc/CommandLine.h"
#include "Misc/Crc.h"
#include "Misc/EngineVersion.h"
#include "Misc/Guid.h"
#include "Misc/NetworkGuid.h" // IWYU pragma: keep
#include "Misc/Parse.h"
#include "Serialization/CustomVersion.h"
#include "Trace/Detail/Channel.h"
#include "Misc/StringBuilder.h"

DEFINE_LOG_CATEGORY( LogNetVersion );

FNetworkVersion::FGetLocalNetworkVersionOverride FNetworkVersion::GetLocalNetworkVersionOverride;
FNetworkVersion::FIsNetworkCompatibleOverride FNetworkVersion::IsNetworkCompatibleOverride;
FNetworkVersion::FGetReplayCompatibleChangeListOverride FNetworkVersion::GetReplayCompatibleChangeListOverride;

namespace UE::Net::Private
{
	FGuid GetEngineNetworkVersionGuid()
	{
		static FGuid Guid = FGuid(0x62915CA3, 0x1C8E4BF7, 0xA30E12C7, 0xC8219DF7);
		return Guid;
	}

	FGuid GetGameNetworkVersionGuid()
	{
		static FGuid Guid = FGuid(0xCC400D24, 0xE0E94E7B, 0x9BF9A283, 0xDCC0C027);
		return Guid;
	}

	FCustomVersionContainer& GetNetworkCustomVersions()
	{
		static FCustomVersionContainer NetworkCustomVersions;

		static bool bInitialized = false;

		if (!bInitialized)
		{
			const FGuid EngineNetworkGuid = UE::Net::Private::GetEngineNetworkVersionGuid();
			const FGuid GameNetworkGuid = UE::Net::Private::GetGameNetworkVersionGuid();

			NetworkCustomVersions.SetVersion(EngineNetworkGuid, FEngineNetworkCustomVersion::LatestVersion, TEXT("EngineNetworkVersion"));
			NetworkCustomVersions.SetVersion(GameNetworkGuid, FGameNetworkCustomVersion::LatestVersion, TEXT("GameNetworkVersion"));

			bInitialized = true;
		}

		return NetworkCustomVersions;
	}

	FCustomVersionContainer& GetCompatibleNetworkCustomVersions()
	{
		static FCustomVersionContainer CompatibleNetworkCustomVersions;

		static bool bInitialized = false;

		if (!bInitialized)
		{
			const FGuid EngineNetworkGuid = UE::Net::Private::GetEngineNetworkVersionGuid();
			const FGuid GameNetworkGuid = UE::Net::Private::GetGameNetworkVersionGuid();

			CompatibleNetworkCustomVersions.SetVersion(EngineNetworkGuid, FEngineNetworkCustomVersion::ReplayBackwardsCompat, TEXT("EngineNetworkVersion"));
			CompatibleNetworkCustomVersions.SetVersion(GameNetworkGuid, FGameNetworkCustomVersion::LatestVersion, TEXT("GameNetworkVersion"));

			bInitialized = true;
		}

		return CompatibleNetworkCustomVersions;
	}
}

const FGuid FEngineNetworkCustomVersion::Guid = UE::Net::Private::GetEngineNetworkVersionGuid();
FCustomVersionRegistration GRegisterEngineNetworkCustomVersion(FEngineNetworkCustomVersion::Guid, FEngineNetworkCustomVersion::LatestVersion, TEXT("EngineNetworkVersion"));

const FGuid FGameNetworkCustomVersion::Guid = UE::Net::Private::GetGameNetworkVersionGuid();
FCustomVersionRegistration GRegisterGameNetworkCustomVersion(FGameNetworkCustomVersion::Guid, FGameNetworkCustomVersion::LatestVersion, TEXT("GameNetworkVersion"));

FString& FNetworkVersion::GetProjectVersion_Internal()
{
	static FString ProjectVersion = "1.0.0";
	return ProjectVersion;
}

bool FNetworkVersion::bHasCachedNetworkChecksum			= false;
uint32 FNetworkVersion::CachedNetworkChecksum			= 0;

bool FNetworkVersion::bHasCachedReplayChecksum = false;
uint32 FNetworkVersion::CachedReplayChecksum = 0;

void FNetworkVersion::SetProjectVersion(const TCHAR* InVersion)
{
	if (ensureMsgf(InVersion != nullptr && FCString::Strlen(InVersion), TEXT("ProjectVersion used for network version must be a valid string!")))
	{
		FString& ProjectVersion = GetProjectVersion_Internal();

		ProjectVersion = InVersion;
		bHasCachedNetworkChecksum = false;

		UE_LOGF(LogNetVersion, Log, "Set ProjectVersion to %ls. Version Checksum will be recalculated on next use.", *ProjectVersion);
	}	
}

void FNetworkVersion::SetGameNetworkProtocolVersion(const uint32 InGameNetworkProtocolVersion)
{
	FCustomVersionContainer& NetworkCustomVersions = UE::Net::Private::GetNetworkCustomVersions();

	NetworkCustomVersions.SetVersion(FGameNetworkCustomVersion::Guid, InGameNetworkProtocolVersion, TEXT("GameNetworkVersion"));
	bHasCachedNetworkChecksum = false;

	UE_LOGF(LogNetVersion, Log, "Set GameNetworkProtocolVersion to %ud. Version Checksum will be recalculated on next use.", InGameNetworkProtocolVersion);
}

void FNetworkVersion::SetGameCompatibleNetworkProtocolVersion(const uint32 InGameCompatibleNetworkProtocolVersion)
{
	FCustomVersionContainer& CompatibleNetworkCustomVersions = UE::Net::Private::GetCompatibleNetworkCustomVersions();

	CompatibleNetworkCustomVersions.SetVersion(FGameNetworkCustomVersion::Guid, InGameCompatibleNetworkProtocolVersion, TEXT("GameNetworkVersion"));
	
	bHasCachedNetworkChecksum = false;
	bHasCachedReplayChecksum = false;

	UE_LOGF(LogNetVersion, Log, "Set GameCompatibleNetworkProtocolVersion to %ud. Version Checksum will be recalculated on next use.", InGameCompatibleNetworkProtocolVersion);
}

uint32 FNetworkVersion::GetNetworkCompatibleChangelist()
{
	static int32 ReturnedVersion = ENGINE_NET_VERSION;
	static bool bStaticCheck = false;

	// add a cvar so it can be modified at runtime
	static FAutoConsoleVariableRef CVarNetworkVersionOverride(
		TEXT("networkversionoverride"), ReturnedVersion,
		TEXT("Sets network version used for multiplayer "),
		ECVF_Default);

	if (!bStaticCheck)
	{
		bStaticCheck = true;
		FParse::Value(FCommandLine::Get(), TEXT("networkversionoverride="), ReturnedVersion);
	}

	// If we have a version set explicitly, use that. Otherwise fall back to the regular engine version changelist, since it might be set at runtime (via Build.version).
	if (ReturnedVersion == 0)
	{
		return ENGINE_NET_VERSION ? ENGINE_NET_VERSION : BuildSettings::GetCompatibleChangelist();
	}

	return (uint32)ReturnedVersion;
}

uint32 FNetworkVersion::GetReplayCompatibleChangelist()
{
	// Use the override if it's bound
	if (GetReplayCompatibleChangeListOverride.IsBound())
	{
		const uint32 Changelist = GetReplayCompatibleChangeListOverride.Execute();

		UE_LOGF(LogNetVersion, Log, "Replay changelist override: %d", Changelist);
		return Changelist;
	}

	return FEngineVersion::CompatibleWith().GetChangelist();
}

uint32 FNetworkVersion::GetNetworkProtocolVersion(const FGuid& VersionGuid)
{
	const FCustomVersionContainer& NetworkCustomVersions = UE::Net::Private::GetNetworkCustomVersions();
	const FCustomVersion* CustomVersion = NetworkCustomVersions.GetVersion(VersionGuid);
	return CustomVersion ? CustomVersion->Version : 0;
}

uint32 FNetworkVersion::GetCompatibleNetworkProtocolVersion(const FGuid& VersionGuid)
{
	const FCustomVersionContainer& CompatibleNetworkCustomVersions = UE::Net::Private::GetCompatibleNetworkCustomVersions();
	const FCustomVersion* CustomVersion = CompatibleNetworkCustomVersions.GetVersion(VersionGuid);
	return CustomVersion ? CustomVersion->Version : 0;
}

const FCustomVersionContainer& FNetworkVersion::GetNetworkCustomVersions()
{
	return UE::Net::Private::GetNetworkCustomVersions();
}

void FNetworkVersion::RegisterNetworkCustomVersion(const FGuid& VersionGuid, int32 Version, int32 CompatibleVersion, const FName& FriendlyName)
{
	FCustomVersionContainer& NetworkCustomVersions = UE::Net::Private::GetNetworkCustomVersions();
	FCustomVersionContainer& CompatibleNetworkCustomVersions = UE::Net::Private::GetCompatibleNetworkCustomVersions();

	if (const FCustomVersion* ExistingVersion = NetworkCustomVersions.GetVersion(VersionGuid))
	{
		if (ExistingVersion->Version != Version)
		{
			UE_LOGF(LogNetVersion, Warning, "RegisterNetworkCustomVersion: Overwriting previous version %d with %d for %ls", ExistingVersion->Version, Version, *FriendlyName.ToString());
		}
	}

	NetworkCustomVersions.SetVersion(VersionGuid, Version, FriendlyName);

	if (const FCustomVersion* ExistingVersion = CompatibleNetworkCustomVersions.GetVersion(VersionGuid))
	{
		if (ExistingVersion->Version != CompatibleVersion)
		{
			UE_LOGF(LogNetVersion, Warning, "RegisterNetworkCustomVersion: Overwriting previous compatible version %d with %d for %ls", ExistingVersion->Version, CompatibleVersion, *FriendlyName.ToString());
		}
	}

	CompatibleNetworkCustomVersions.SetVersion(VersionGuid, CompatibleVersion, FriendlyName);

	bHasCachedNetworkChecksum = false;
	bHasCachedReplayChecksum = false;
}

uint32 FNetworkVersion::GetLocalNetworkVersion( bool AllowOverrideDelegate /*=true*/ )
{
	if ( bHasCachedNetworkChecksum )
	{
		return CachedNetworkChecksum;
	}

	if ( AllowOverrideDelegate && GetLocalNetworkVersionOverride.IsBound() )
	{
		CachedNetworkChecksum = GetLocalNetworkVersionOverride.Execute();

		UE_LOGF( LogNetVersion, Log, "Checksum from delegate: %u", CachedNetworkChecksum );

		bHasCachedNetworkChecksum = true;

		return CachedNetworkChecksum;
	}

	FString VersionString = FString::Printf(TEXT("%s %s, NetCL: %d"),
		FApp::GetProjectName(),
		*FNetworkVersion::GetProjectVersion(),
		GetNetworkCompatibleChangelist());

	const FCustomVersionContainer& NetworkCustomVersions = UE::Net::Private::GetNetworkCustomVersions();

	const FCustomVersionArray& CustomVers = NetworkCustomVersions.GetAllVersions();
	for (const FCustomVersion& CustomVer : CustomVers)
	{
		VersionString += FString::Printf(TEXT(", %s: %d"), *CustomVer.GetFriendlyName().ToString(), CustomVer.Version);
	}

	CachedNetworkChecksum = FCrc::StrCrc32(*VersionString.ToLower());

	UE_LOGF(LogNetVersion, Log, "%ls (Checksum: %u)", *VersionString, CachedNetworkChecksum);

	bHasCachedNetworkChecksum = true;

	return CachedNetworkChecksum;
}

bool FNetworkVersion::IsNetworkCompatible( const uint32 LocalNetworkVersion, const uint32 RemoteNetworkVersion )
{
	if ( IsNetworkCompatibleOverride.IsBound() )
	{
		return IsNetworkCompatibleOverride.Execute( LocalNetworkVersion, RemoteNetworkVersion );
	}

	return LocalNetworkVersion == RemoteNetworkVersion;
}

FNetworkReplayVersion FNetworkVersion::GetReplayVersion()
{
	if (!bHasCachedReplayChecksum)
	{
		FString VersionString = FApp::GetProjectName();

		const FCustomVersionContainer& CompatibleNetworkCustomVersions = UE::Net::Private::GetCompatibleNetworkCustomVersions();

		const FCustomVersionArray& CustomVers = CompatibleNetworkCustomVersions.GetAllVersions();
		for (const FCustomVersion& CustomVer : CustomVers)
		{
			VersionString += FString::Printf(TEXT(", %s: %d"), *CustomVer.GetFriendlyName().ToString(), CustomVer.Version);
		}

		CachedReplayChecksum = FCrc::StrCrc32(*VersionString.ToLower());
		
		bHasCachedReplayChecksum = true;
	}

	return FNetworkReplayVersion(FApp::GetProjectName(), CachedReplayChecksum, GetReplayCompatibleChangelist());
}

bool FNetworkVersion::AreNetworkRuntimeFeaturesCompatible(EEngineNetworkRuntimeFeatures LocalFeatures, EEngineNetworkRuntimeFeatures RemoteFeatures)
{
	return LocalFeatures == RemoteFeatures;
}

void FNetworkVersion::DescribeNetworkRuntimeFeaturesBitset(EEngineNetworkRuntimeFeatures FeaturesBitflag, FStringBuilderBase& OutVerboseDescription)
{
	if (EnumHasAnyFlags(FeaturesBitflag, EEngineNetworkRuntimeFeatures::IrisEnabled))
	{
		OutVerboseDescription.Append(TEXT("IrisReplication"));
	}
	else
	{
		OutVerboseDescription.Append(TEXT("GenericReplication"));
	}
	if (EnumHasAnyFlags(FeaturesBitflag, EEngineNetworkRuntimeFeatures::UseRemoteObjectReferences))
	{
		OutVerboseDescription.Append(TEXT(", UseRemoteObjectReferences"));
	}
}