// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/NotNull.h"

#include "CoreGlobals.h"
#include "Serialization/Archive.h"

#if UE_ENABLE_NOTNULL_WRAPPER && DO_CHECK

FORCENOINLINE void UE::Core::Private::ReportNotNullPtr()
{
	UE_LOGF(LogCore, Fatal, "Null assigned to TNotNull");
	for (;;);
}

FORCENOINLINE void UE::Core::Private::CheckLoadingNotNullPtr(FArchive& Ar)
{
	if (Ar.IsLoading())
	{
		UE_LOGF(LogCore, Fatal, "Null assigned to TNotNull while reading from archive '%ls'", *Ar.GetArchiveName());
	}
}

#endif // UE_ENABLE_NOTNULL_WRAPPER && DO_CHECK
