// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/ObjectThumbnail.h"
#include "Serialization/StructuredArchive.h"
//#include "ImageCore.h"

#define USE_JPEG_FOR_THUMBNAILS 1

/** Static: Thumbnail compressor */
// these are installed from UnrealEngine.cpp startup
// because we are in Core Module we can't access ImageWrapper ourselves
FThumbnailCompressionInterface* FObjectThumbnail::PNGThumbnailCompressor = nullptr;
FThumbnailCompressionInterface* FObjectThumbnail::JPEGThumbnailCompressor = nullptr;


FObjectThumbnail::FObjectThumbnail()
	: ImageWidth( 0 ),
	  ImageHeight( 0 ),
	  bIsDirty( false ),
	  bLoadedFromDisk(false),
	  bIsJPEG(false),
	  bCreatedAfterCustomThumbForSharedTypesEnabled(false)
{ }


const TArray< uint8 >& FObjectThumbnail::GetUncompressedImageData() const
{
	if( ImageData.Num() == 0 )
	{
		// Const cast here so that we can populate the image data on demand	(write once)
		FObjectThumbnail* MutableThis = const_cast< FObjectThumbnail* >( this );
		MutableThis->DecompressImageData();
	}
	return ImageData;
}

void FObjectThumbnail::Serialize( FArchive& Ar )
{
	Serialize(FStructuredArchiveFromArchive(Ar).GetSlot());
}

void FObjectThumbnail::Serialize(FStructuredArchive::FSlot Slot)
{
	//if the image thinks it's empty, ensure there is no memory waste
	if ((ImageWidth == 0) || (ImageHeight == 0))
	{
		CompressedImageData.Reset();
	}

	// Compress the image on demand if we don't have any compressed bytes yet.
	if (CompressedImageData.Num() == 0 &&
		(Slot.GetUnderlyingArchive().IsSaving() || Slot.GetUnderlyingArchive().IsCountingMemory()))
	{
		CompressImageData();
	}

	FStructuredArchive::FRecord Record = Slot.EnterRecord();
	Record << SA_VALUE(TEXT("ImageWidth"), ImageWidth);
	
	// serialize bIsJPEG via negative ImageHeight
	if (Slot.GetArchiveState().IsLoading())
	{
		Record << SA_VALUE(TEXT("ImageHeight"), ImageHeight);
		bIsJPEG = false;
		if ( ImageHeight < 0 )
		{
			bIsJPEG = true;
			ImageHeight = -ImageHeight;
		}
	}
	else
	{
		int32 ImageHeightToSerialize = bIsJPEG ? -ImageHeight : ImageHeight;
		
		Record << SA_VALUE(TEXT("ImageHeight"), ImageHeightToSerialize);
	}
	
	if (Slot.GetArchiveState().IsLoading())
	{
		bLoadedFromDisk = true;
		bCreatedAfterCustomThumbForSharedTypesEnabled = true;

		if ((ImageWidth == 0) || (ImageHeight == 0))
		{
			// The thumbnail data doesn't exist and might be corrupt. Attempting to read the
			// image data may lead to crashes. Mark the thumbnail dirtsy so we may regenerate it
			bIsDirty = true; 
			return;
		}
	}

	Record << SA_VALUE(TEXT("CompressedImageData"), CompressedImageData);

	if (Slot.GetUnderlyingArchive().IsCountingMemory())
	{
		Record << SA_VALUE(TEXT("ImageData"), ImageData) << SA_VALUE(TEXT("bIsDirty"), bIsDirty);
	}
}


void FObjectThumbnail::CompressImageData()
{
	CompressedImageData.Reset();
	bIsJPEG = false;

	if (ImageData.Num() > 0)
	{
		if (FThumbnailCompressionInterface* Compressor = ChooseNewCompressor())
		{
			Compressor->CompressImage(ImageData, ImageWidth, ImageHeight, CompressedImageData);

			if (Compressor == JPEGThumbnailCompressor)
			{
				bIsJPEG = true;
			}
		}
	}
}

void FObjectThumbnail::DecompressImageData()
{
	ImageData.Reset();

	if (FThumbnailCompressionInterface* Compressor = GetCompressor())
	{
		Compressor->DecompressImage(CompressedImageData, ImageWidth, ImageHeight, ImageData);
	}
}

void FObjectThumbnail::EmptyUncompressedImageData()
{
	ImageData.Empty();
}

FThumbnailCompressionInterface* FObjectThumbnail::GetCompressor() const
{
	if (!CompressedImageData.IsEmpty() && ImageWidth > 0 && ImageHeight > 0)
	{
		if (bIsJPEG)
		{
			return JPEGThumbnailCompressor;
		}
		else
		{
			return PNGThumbnailCompressor;
		}
	}

	return nullptr;
}

FThumbnailCompressionInterface* FObjectThumbnail::ChooseNewCompressor() const
{
	if (ImageWidth > 0 && ImageHeight > 0)
	{
		#if USE_JPEG_FOR_THUMBNAILS
		// prefer JPEG except on images that are trivially tiny or 1d
		if (JPEGThumbnailCompressor && ImageWidth >= 8 && ImageHeight >= 8)
		{
			return JPEGThumbnailCompressor;
		}
		#endif

		if (PNGThumbnailCompressor)
		{
			return PNGThumbnailCompressor;
		}
	}

	return nullptr;
}

void FObjectThumbnail::CountBytes( FArchive& Ar ) const
{
	SIZE_T StaticSize = sizeof(FObjectThumbnail);
	Ar.CountBytes(StaticSize, Align(StaticSize, alignof(FObjectThumbnail)));

	FObjectThumbnail* UnconstThis = const_cast<FObjectThumbnail*>(this);
	UnconstThis->CompressedImageData.CountBytes(Ar);
	UnconstThis->ImageData.CountBytes(Ar);
}


void FObjectThumbnail::CountImageBytes_Compressed( FArchive& Ar ) const
{
	const_cast<FObjectThumbnail*>(this)->CompressedImageData.CountBytes(Ar);
}


void FObjectThumbnail::CountImageBytes_Uncompressed( FArchive& Ar ) const
{
	const_cast<FObjectThumbnail*>(this)->ImageData.CountBytes(Ar);
}


void FObjectFullNameAndThumbnail::CountBytes( FArchive& Ar ) const
{
	SIZE_T StaticSize = sizeof(FObjectFullNameAndThumbnail);
	Ar.CountBytes(StaticSize, Align(StaticSize, alignof(FObjectFullNameAndThumbnail)));

	if ( ObjectThumbnail != NULL )
	{
		ObjectThumbnail->CountBytes(Ar);
	}
}
