// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/OutputDevice.h"

#include "Containers/UnrealString.h"
#include "CoreGlobals.h"
#include "HAL/PlatformTime.h"
#include "Internationalization/Text.h"
#include "Logging/LogMacros.h"
#include "Logging/LogScopedCategoryAndVerbosityOverride.h"
#include "Logging/StructuredLog.h"
#include "Misc/StringBuilder.h"
#include "Misc/VarargsHelper.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

class FOutputDeviceError;

DEFINE_LOG_CATEGORY(LogOutputDevice);

FOutputDevice::FOutputDevice()
		: bSuppressEventTag      (false)
		, bAutoEmitLineTerminator(true)
{}

FOutputDevice::FOutputDevice(FOutputDevice&&) = default;
FOutputDevice::FOutputDevice(const FOutputDevice&) = default;
FOutputDevice& FOutputDevice::operator=(FOutputDevice&&) = default;
FOutputDevice& FOutputDevice::operator=(const FOutputDevice&) = default;
FOutputDevice::~FOutputDevice() = default;

void FOutputDevice::Log( ELogVerbosity::Type Verbosity, const TCHAR* Str )
{
	Serialize( Str, Verbosity, NAME_None );
}
void FOutputDevice::Log( ELogVerbosity::Type Verbosity, const FString& S )
{
	Serialize( *S, Verbosity, NAME_None );
}
void FOutputDevice::Log( const class FName& Category, ELogVerbosity::Type Verbosity, const TCHAR* Str )
{
	Serialize( Str, Verbosity, Category );
}
void FOutputDevice::Log( const class FName& Category, ELogVerbosity::Type Verbosity, const FString& S )
{
	Serialize( *S, Verbosity, Category );
}
void FOutputDevice::Log( const TCHAR* Str )
{
	FScopedCategoryAndVerbosityOverride::FOverride* TLS = FScopedCategoryAndVerbosityOverride::GetTLSCurrent();
	Serialize( Str, TLS->Verbosity, TLS->Category );
}
void FOutputDevice::Log( const FString& S )
{
	FScopedCategoryAndVerbosityOverride::FOverride* TLS = FScopedCategoryAndVerbosityOverride::GetTLSCurrent();
	Serialize( *S, TLS->Verbosity, TLS->Category );
}
void FOutputDevice::Log( const FText& T )
{
	FScopedCategoryAndVerbosityOverride::FOverride* TLS = FScopedCategoryAndVerbosityOverride::GetTLSCurrent();
	Serialize( *T.ToString(), TLS->Verbosity, TLS->Category );
}
/*-----------------------------------------------------------------------------
	Formatted printing and messages.
-----------------------------------------------------------------------------*/

// Do not inline these functions, in case we need to capture a call stack in FOutputDeviceError::Serialize. We need to be certain of how many frames to ignore.
FORCENOINLINE void FOutputDevice::CategorizedLogfImpl(const FName& Category, ELogVerbosity::Type Verbosity, const TCHAR* Fmt, ...)
{
	GROWABLE_LOGF(Serialize(Buffer, Verbosity, Category))
}
FORCENOINLINE void FOutputDevice::LogfImpl(ELogVerbosity::Type Verbosity, const TCHAR* Fmt, ...)
{
	// call serialize with the final buffer
	GROWABLE_LOGF(Serialize(Buffer, Verbosity, NAME_None))
}
FORCENOINLINE void FOutputDevice::LogfImpl(const TCHAR* Fmt, ...)
{
	FScopedCategoryAndVerbosityOverride::FOverride* TLS = FScopedCategoryAndVerbosityOverride::GetTLSCurrent();
	GROWABLE_LOGF(Serialize(Buffer, TLS->Verbosity, TLS->Category))
}

void FOutputDevice::SerializeRecord(const UE::FLogRecord& Record)
{
	TStringBuilder<512> Text;
	Record.FormatMessageTo(Text);
	Serialize(*Text, Record.GetVerbosity(), Record.GetCategory());
}

/** Critical errors. */
CORE_API FOutputDeviceError* GError = NULL;
