// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/OutputDeviceAnsiError.h"
#include "Templates/UnrealTemplate.h"
#include "Logging/LogMacros.h"
#include "CoreGlobals.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Misc/CoreDelegates.h"
#include "Misc/App.h"

/** Constructor, initializing member variables */
FOutputDeviceAnsiError::FOutputDeviceAnsiError()
:	ErrorPos(0)
{}

/**
 * Serializes the passed in data unless the current event is suppressed.
 *
 * @param	Data	Text to log
 * @param	Event	Event name used for suppression purposes
 */
void FOutputDeviceAnsiError::Serialize( const TCHAR* Msg, ELogVerbosity::Type Verbosity, const class FName& Category )
{
	// Display the error and exit.
	FPlatformMisc::LocalPrint( TEXT("\nappError called: \n") );
	FPlatformMisc::LocalPrint( Msg );
	FPlatformMisc::LocalPrint( TEXT("\n") );

	if( !GIsCriticalError )
	{
		// First appError.
		GIsCriticalError = 1;
		UE_LOGF(LogHAL, Error, "appError called: %ls", Msg );
		FCString::Strncpy( GErrorHist, Msg, UE_ARRAY_COUNT(GErrorHist) );
		FCString::StrncatTruncateDest( GErrorHist, UE_ARRAY_COUNT(GErrorHist), TEXT("\r\n\r\n") );
		ErrorPos = FCString::Strlen(GErrorHist);
	}
	else
	{
		UE_LOGF(LogHAL, Error, "Error reentered: %ls", Msg );
	}

	UE_DEBUG_BREAK();

	if( GIsGuarded )
	{
		// Propagate error so structured exception handler can perform necessary work.
#if PLATFORM_EXCEPTIONS_DISABLED
		UE_DEBUG_BREAK();
#endif
		FPlatformMisc::RaiseException( 1 );
	}
	else
	{
		// We crashed outside the guarded code (e.g. appExit).
		HandleError();
		// pop up a crash window if we are not in unattended mode
		if( FApp::IsUnattended() == false )
		{
			FPlatformMisc::RequestExit( true, TEXT("FOutputDeviceAnsiError::Serialize.!GIsGuarded"));
		}
		else
		{
			UE_LOGF(LogHAL, Error, "%ls", Msg );
		}		
	}
}

/**
 * Error handling function that is being called from within the system wide global
 * error handler, e.g. using structured exception handling on the PC.
 */
void FOutputDeviceAnsiError::HandleError()
{
	GIsGuarded			= 0;
	GIsRunning			= 0;
	GIsCriticalError	= 1;
	GLogConsole			= NULL;
	GErrorHist[UE_ARRAY_COUNT(GErrorHist)-1] = TCHAR('\0');

	if (GLog)
	{
		// print to log and flush it
		UE_LOG( LogHAL, Log, TEXT( "=== Critical error: ===" LINE_TERMINATOR_ANSI LINE_TERMINATOR_ANSI "%s" LINE_TERMINATOR_ANSI), GErrorExceptionDescription );
		UE_LOGF(LogHAL, Log, "%ls", GErrorHist);

		GLog->Panic();
	}
	else
	{
		FPlatformMisc::LocalPrint( GErrorHist );
	}

	FPlatformMisc::LocalPrint( TEXT("\n\nExiting due to error\n") );

	FCoreDelegates::OnShutdownAfterError.Broadcast();
}
