// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/OutputDeviceArchiveWrapper.h"
#include "Misc/OutputDeviceHelper.h"
#include "Serialization/Archive.h"

void FOutputDeviceArchiveWrapper::Flush()
{
	LogAr->Flush();
}

void FOutputDeviceArchiveWrapper::Serialize(const TCHAR* Data, ELogVerbosity::Type Verbosity, const class FName& Category)
{
	if (Verbosity != ELogVerbosity::SetColor)
	{
		FOutputDeviceHelper::FormatCastAndSerializeLine(*LogAr, Data, Verbosity, Category, -1.0, true, bAutoEmitLineTerminator);
	}
}

