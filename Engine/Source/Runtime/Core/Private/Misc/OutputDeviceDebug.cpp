// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/OutputDeviceDebug.h"

#include "Containers/StringFwd.h"
#include "CoreGlobals.h"
#include "HAL/PlatformMisc.h"
#include "HAL/PlatformTime.h"
#include "Logging/StructuredLog.h"
#include "Misc/DateTime.h"
#include "Misc/OutputDeviceHelper.h"
#include "Misc/StringBuilder.h"
#include "Misc/TVariant.h"
#include "Serialization/CompactBinary.h"
#include "Serialization/CompactBinarySerialization.h"
#include "Serialization/CompactBinaryWriter.h"

/** Seconds between calls to FOutputDeviceDebug::TickAsync. */
constexpr static double GOutputDeviceDebugTickPeriod = 1.0;

void FOutputDeviceDebug::Serialize(const TCHAR* Data, ELogVerbosity::Type Verbosity, const FName& Category, double Time)
{
	if (Verbosity == ELogVerbosity::SetColor)
	{
		return;
	}

	ConditionalTickAsync(Time >= 0.0 ? Time : FPlatformTime::Seconds() - GStartTime);

	if (bSerializeAsJson.load(std::memory_order_relaxed))
	{
		TVariant<TCbWriter<1024>, TStringBuilder<512>> WriterMemory;

		WriterMemory.Emplace<TCbWriter<1024>>();
		TCbWriter<1024>& Writer = WriterMemory.Get<TCbWriter<1024>>();
		Writer.BeginObject();
		Writer.AddDateTime(ANSITEXTVIEW("Time"), FDateTime::UtcNow());
		Writer.AddString(ANSITEXTVIEW("Category"), WriteToUtf8String<64>(Category));
		Writer.AddString(ANSITEXTVIEW("Verbosity"), ToString(Verbosity));
		Writer.AddString(ANSITEXTVIEW("Message"), Data);
		Writer.EndObject();

		TArray<uint8, TInlineAllocator64<1024>> Buffer;
		Buffer.AddUninitialized((int64)Writer.GetSaveSize());
		FCbFieldView Object = Writer.Save(MakeMemoryView(Buffer));

		WriterMemory.Emplace<TStringBuilder<512>>();
		TStringBuilder<512>& Json = WriterMemory.Get<TStringBuilder<512>>();
		CompactBinaryToCompactJson(Object, Json);
		Json.Append(LINE_TERMINATOR);
		FPlatformMisc::LowLevelOutputDebugString(*Json);
	}
	else
	{
		TStringBuilder<512> Line;
		FOutputDeviceHelper::AppendFormatLogLine(Line, Verbosity, Category, Data, GPrintLogTimes, Time);
		Line.Append(LINE_TERMINATOR);
		FPlatformMisc::LowLevelOutputDebugString(*Line);
	}
}

void FOutputDeviceDebug::Serialize(const TCHAR* Data, ELogVerbosity::Type Verbosity, const FName& Category)
{
	Serialize(Data, Verbosity, Category, -1.0);
}

void FOutputDeviceDebug::SerializeRecord(const UE::FLogRecord& Record)
{
	if (Record.GetVerbosity() == ELogVerbosity::SetColor)
	{
		return;
	}

	ConditionalTickAsync(FPlatformTime::Seconds() - GStartTime);

	if (bSerializeAsJson.load(std::memory_order_relaxed))
	{
		TVariant<TUtf8StringBuilder<1024>, TArray<uint8, TInlineAllocator64<1024>>> ScratchMemory;
		TVariant<TCbWriter<1024>, TStringBuilder<512>> WriterMemory;

		ScratchMemory.Emplace<TUtf8StringBuilder<1024>>();
		TUtf8StringBuilder<1024>& Message = ScratchMemory.Get<TUtf8StringBuilder<1024>>();
		Record.FormatMessageTo(Message);

		WriterMemory.Emplace<TCbWriter<1024>>();
		TCbWriter<1024>& Writer = WriterMemory.Get<TCbWriter<1024>>();
		Writer.BeginObject();
		Writer.AddDateTime(ANSITEXTVIEW("Time"), Record.GetTime().GetUtcTime());
		Writer.AddString(ANSITEXTVIEW("Category"), WriteToUtf8String<64>(Record.GetCategory()));
		Writer.AddString(ANSITEXTVIEW("Verbosity"), ToString(Record.GetVerbosity()));
		Writer.AddString(ANSITEXTVIEW("Message"), Message);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS;
		if (const UTF8CHAR* Utf8Format = Record.GetUtf8Format())
		{
			Writer.AddString(ANSITEXTVIEW("Format"), Utf8Format);
		}
		else if (const TCHAR* Format = Record.GetFormat())
		{
			Writer.AddString(ANSITEXTVIEW("Format"), Format);
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS;
		if (const ANSICHAR* File = Record.GetFile())
		{
			Writer.AddString(ANSITEXTVIEW("File"), File);
		}
		if (int32 Line = Record.GetLine())
		{
			Writer.AddInteger(ANSITEXTVIEW("Line"), Line);
		}
		if (const FCbObject& Fields = Record.GetFields())
		{
			Writer.AddObject(ANSITEXTVIEW("Fields"), Fields);
		}
		Writer.EndObject();

		ScratchMemory.Emplace<TArray<uint8, TInlineAllocator64<1024>>>();
		TArray<uint8, TInlineAllocator64<1024>>& Buffer = ScratchMemory.Get<TArray<uint8, TInlineAllocator64<1024>>>();
		Buffer.AddUninitialized((int64)Writer.GetSaveSize());
		FCbFieldView Object = Writer.Save(MakeMemoryView(Buffer));

		WriterMemory.Emplace<TStringBuilder<512>>();
		TStringBuilder<512>& Json = WriterMemory.Get<TStringBuilder<512>>();
		CompactBinaryToCompactJson(Object, Json);
		Json.Append(LINE_TERMINATOR);
		FPlatformMisc::LowLevelOutputDebugString(*Json);
	}
	else
	{
		TStringBuilder<512> Line;
		FOutputDeviceHelper::AppendFormatLogLine(Line, Record.GetVerbosity(), Record.GetCategory(), nullptr, GPrintLogTimes);
		Record.FormatMessageTo(Line);
		Line.Append(LINE_TERMINATOR);
		FPlatformMisc::LowLevelOutputDebugString(*Line);
	}
}

bool FOutputDeviceDebug::CanBeUsedOnMultipleThreads() const
{
	return FPlatformMisc::IsLocalPrintThreadSafe();
}

FORCEINLINE void FOutputDeviceDebug::ConditionalTickAsync(double Time)
{
	double LocalLastTickTime = LastTickTime.load(std::memory_order_relaxed);
	if ((LocalLastTickTime == 0.0 || Time >= LocalLastTickTime + GOutputDeviceDebugTickPeriod) &&
		LastTickTime.compare_exchange_strong(LocalLastTickTime, Time))
	{
		TickAsync();
	}
}

void FOutputDeviceDebug::TickAsync()
{
	bSerializeAsJson = FPlatformMisc::IsLowLevelOutputDebugStringStructured();
}
