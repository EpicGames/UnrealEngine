// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/OutputDeviceFile.h"
#include "Misc/AssertionMacros.h"
#include "HAL/UnrealMemory.h"
#include "Templates/UnrealTemplate.h"
#include "Serialization/Archive.h"
#include "Containers/Array.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "Containers/Set.h"
#include "Misc/DateTime.h"
#include "HAL/PlatformTime.h"
#include "HAL/PlatformProcess.h"
#include "HAL/PlatformOutputDevices.h"
#include "CoreGlobals.h"
#include "HAL/FileManager.h"
#include "Misc/Parse.h"
#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "HAL/ThreadSafeCounter.h"
#include "Misc/ScopeLock.h"
#include "Misc/CommandLine.h"
#include "Misc/Paths.h"
#include "Misc/OutputDeviceHelper.h"
#include "Math/Color.h"
#include "Templates/Atomic.h"
#include "HAL/ConsoleManager.h"
#include "Misc/Fork.h"
#include "Stats/Stats.h"

#include <atomic>

/** Used by tools which include only core to disable log file creation. */
#ifndef ALLOW_LOG_FILE
	#define ALLOW_LOG_FILE 1
#endif

#ifndef OUTPUTDEVICE_DEFAULT_ASYNC_WRITER_THREAD_NAME_OPTION
	#define OUTPUTDEVICE_DEFAULT_ASYNC_WRITER_THREAD_NAME_OPTION FAsyncWriter::EThreadNameOption::FileName
#endif

typedef uint8 UTF8BOMType[3];
static UTF8BOMType UTF8BOM = { 0xEF, 0xBB, 0xBF };

static float GLogFlushIntervalSec = 0.2f;
static FAutoConsoleVariableRef CVarLogFlushInterval(
	TEXT("log.flushInterval"),
	GLogFlushIntervalSec,
	TEXT("Logging interval in seconds"),
	ECVF_Default );


#if UE_BUILD_SHIPPING
static float GLogFlushIntervalSec_Shipping = 0.0f;
static FAutoConsoleVariableRef CVarLogFlushIntervalShipping(
	TEXT("log.flushInterval.Shipping"),
	GLogFlushIntervalSec_Shipping,
	TEXT("Logging interval in shipping. If set, this overrides archive.FlushInterval"),
	ECVF_Default);
#endif

inline double GetLogFlushIntervalSec()
{
#if UE_BUILD_SHIPPING
	return double((GLogFlushIntervalSec_Shipping>0.0f) ? GLogFlushIntervalSec_Shipping : GLogFlushIntervalSec);
#else
	return double(GLogFlushIntervalSec);
#endif
}


/** [WRITER THREAD] Flushes the archive and reset the flush timer. */
void FAsyncWriter::FlushArchiveAndResetTimer()
{
	// This should be the one and only place where we flush because we want the flush to happen only on the 
	// async writer thread (if threading is enabled)
	Ar.Flush();
	LastArchiveFlushTime = FPlatformTime::Seconds();
}

/** [WRITER THREAD] Serialize the contents of the ring buffer to disk */
void FAsyncWriter::SerializeBufferToArchive()
{
	// Some platforms use FPlatformMallocCrash during a crash, which means this function is not allowed to perform any allocations
	// or else it will deadlock when flushing logs during crash handling. Ideally scoped counters and events would be disabled while crashing.
	// GIsCriticalError is not always true when crashing (i.e. the case of a GPF) so there is no way to know to skip this behavior only when crashing.
#if PLATFORM_ALLOW_ALLOCATIONS_IN_FASYNCWRITER_SERIALIZEBUFFERTOARCHIVE
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FAsyncWriter_SerializeBufferToArchive);
	SCOPED_NAMED_EVENT(FAsyncWriter_SerializeBufferToArchive, FColor::Cyan);
#endif
	while (SerializeRequestCounter.GetValue() > 0)
	{
		// Grab a local copy of the end pos. It's ok if it changes on the client thread later on.
		// We won't be modifying it anyway and will later serialize new data in the next iteration.
		// Here we only serialize what we know exists at the beginning of this function.
		int32 ThisThreadStartPos = BufferStartPos.Load(EMemoryOrder::Relaxed);
		int32 ThisThreadEndPos   = BufferEndPos.Load(EMemoryOrder::SequentiallyConsistent);

		if (ThisThreadEndPos >= ThisThreadStartPos)
		{
			Ar.Serialize(Buffer.GetData() + ThisThreadStartPos, ThisThreadEndPos - ThisThreadStartPos);
		}
		else
		{
			// Data is wrapped around the ring buffer
			Ar.Serialize(Buffer.GetData() + ThisThreadStartPos, Buffer.Num() - ThisThreadStartPos);
			Ar.Serialize(Buffer.GetData(), ThisThreadEndPos);
		}

		// Modify the start pos. Only the worker thread modifies this value so it's ok to not guard it with a critical section.
		BufferStartPos.Store(ThisThreadEndPos, EMemoryOrder::Relaxed);

		// Decrement the request counter, we now know we serialized at least one request.
		// We might have serialized more requests but it's irrelevant, the counter will go down to 0 eventually
		SerializeRequestCounter.Decrement();

		// Flush the archive periodically if running on a separate thread
		if (Thread)
		{
			if ((FPlatformTime::Seconds() - LastArchiveFlushTime) > GetLogFlushIntervalSec() )
			{
				FlushArchiveAndResetTimer();
			}
		}
		// If no threading is available or when we explicitly requested flush (see FlushBuffer), flush immediately after writing.
		// In some rare cases we may flush twice (see above) but that's ok. We need a clear division between flushing because of the timer
		// and force flush on demand.
		if (WantsArchiveFlush.GetValue() > 0)
		{
			FlushArchiveAndResetTimer();
			int32 FlushCount = WantsArchiveFlush.Decrement();
			check(FlushCount >= 0);
		}
	}
}

/** [CLIENT THREAD] Flush the memory buffer (doesn't force the archive to flush). Can only be used from inside of BufferPosCritical lock. */
void FAsyncWriter::FlushBuffer()
{
	SerializeRequestCounter.Increment();
	if (!Thread)
	{
		SerializeBufferToArchive();
	}
	else if (RunCritical.TryLock())
	{
		// The thread object is there, but hasn't started yet (could be a forkable thread pre-fork).
		SerializeBufferToArchive();
		RunCritical.Unlock();
	}
	while (SerializeRequestCounter.GetValue() != 0)
	{
		FPlatformProcess::SleepNoStats(0);
	}
	// Make sure there's been no unexpected concurrency
	check(SerializeRequestCounter.GetValue() == 0);
}

FAsyncWriter::FAsyncWriter(FArchive& InAr, FAsyncWriter::EThreadNameOption NameOption)
	: Thread(nullptr)
	, Ar(InAr)
	, BufferStartPos(0)
	, BufferEndPos(0)
	, LastArchiveFlushTime(0.0)
{
	constexpr int32 InitialBufferSize = 128 * 1024;
	Buffer.AddUninitialized(InitialBufferSize);

	float CommandLineInterval = 0.0;
	if (FParse::Value(FCommandLine::Get(), TEXT("LOGFLUSHINTERVAL="), CommandLineInterval))
	{
		GLogFlushIntervalSec = CommandLineInterval;
	}

	const bool bDisableForkedOutputThread = FParse::Param(FCommandLine::Get(), TEXT("DisableForkedOutputThread"));

	if (!bDisableForkedOutputThread)
	{
		if (FPlatformProcess::SupportsMultithreading() || FForkProcessHelper::SupportsMultithreadingPostFork())
		{
			FString WriterName;
			
			if (NameOption == EThreadNameOption::FileName)
			{
				WriterName = FString::Printf(TEXT("FAsyncWriter_%s"), *FPaths::GetBaseFilename(Ar.GetArchiveName()));
			}
			else if (NameOption == EThreadNameOption::Sequential)
			{
				static std::atomic<uint64> AsyncWriterCount(0);
				WriterName = FString::Printf(TEXT("FAsyncWriter_%llu"), ++AsyncWriterCount);
			}

			int32 StackSize = 32 * 1024;
			FPlatformAtomics::InterlockedExchangePtr((void**)&Thread, FForkProcessHelper::CreateForkableThread(this, *WriterName, StackSize, TPri_BelowNormal));
		}
	}
}

FAsyncWriter::~FAsyncWriter()
{
	Flush();
	delete Thread;
	Thread = nullptr;
}

/** [CLIENT THREAD] Serialize data to buffer that will later be saved to disk by the async thread */
void FAsyncWriter::Serialize(void* InData, int64 Length)
{
	if (!InData || Length <= 0)
	{
		return;
	}

	const uint8* Data = (uint8*)InData;

	FScopeLock WriteLock(&BufferPosCritical);

	const int32 ThisThreadEndPos = BufferEndPos.Load(EMemoryOrder::Relaxed);

	// Store the local copy of the current buffer start pos. It may get moved by the worker thread but we don't
	// care about it too much because we only modify BufferEndPos. Copy should be atomic enough. We only use it
	// for checking the remaining space in the buffer so underestimating is ok.
	{
		const int32 ThisThreadStartPos = BufferStartPos.Load(EMemoryOrder::Relaxed);
		// Calculate the remaining size in the ring buffer
		const int32 BufferFreeSize = ThisThreadStartPos <= ThisThreadEndPos ? (Buffer.Num() - ThisThreadEndPos + ThisThreadStartPos) : (ThisThreadStartPos - ThisThreadEndPos);
		// Make sure the buffer is BIGGER than we require otherwise we may calculate the wrong (0) buffer EndPos for StartPos = 0 and Length = Buffer.Num()
		if (BufferFreeSize <= Length)
		{
			// Force the async thread to call SerializeBufferToArchive even if it's currently empty
			FlushBuffer();

			// Resize the buffer if needed
			if (Length >= Buffer.Num())
			{
				// Keep the buffer bigger than we require so that % Buffer.Num() does not return 0 for Lengths = Buffer.Num()
				Buffer.SetNumUninitialized((int32)(Length + 1));
			}
		}
	}

	// We now know there's enough space in the buffer to copy data
	const int32 WritePos = ThisThreadEndPos;
	if ((WritePos + Length) <= Buffer.Num())
	{
		// Copy straight into the ring buffer
		FMemory::Memcpy(Buffer.GetData() + WritePos, Data, Length);
	}
	else
	{
		// Wrap around the ring buffer
		int32 BufferSizeToEnd = Buffer.Num() - WritePos;
		FMemory::Memcpy(Buffer.GetData() + WritePos, Data, BufferSizeToEnd);
		FMemory::Memcpy(Buffer.GetData(), Data + BufferSizeToEnd, Length - BufferSizeToEnd);
	}

	// Update the end position and let the async thread know we need to write to disk
	BufferEndPos.Store((ThisThreadEndPos + Length) % Buffer.Num(), EMemoryOrder::SequentiallyConsistent);
	SerializeRequestCounter.Increment();

	// No async thread? Serialize now.
	if (!Thread)
	{
		SerializeBufferToArchive();
	}
}

/** Flush all buffers to disk */
void FAsyncWriter::Flush()
{
	FScopeLock WriteLock(&BufferPosCritical);
	WantsArchiveFlush.Increment();
	FlushBuffer();
}

//~ Begin FRunnable Interface.
bool FAsyncWriter::Init()
{
	return true;
}
	
uint32 FAsyncWriter::Run()
{
	FScopeLock RunLock(&RunCritical);

	while (StopTaskCounter.GetValue() == 0)
	{
		if (SerializeRequestCounter.GetValue() > 0)
		{
			SerializeBufferToArchive();
		}
		else if ((FPlatformTime::Seconds() - LastArchiveFlushTime) > GetLogFlushIntervalSec() )
		{
			FlushArchiveAndResetTimer();
		}
		else
		{
			FPlatformProcess::SleepNoStats(0.01f);
		}
	}
	return 0;
}

void FAsyncWriter::Stop()
{
	StopTaskCounter.Increment();
}

void FAsyncWriter::Tick()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FAsyncWriter_Tick);

	if (SerializeRequestCounter.GetValue() > 0)
	{
		SerializeBufferToArchive();
	}
	else if ((FPlatformTime::Seconds() - LastArchiveFlushTime) > GetLogFlushIntervalSec())
	{
		FlushArchiveAndResetTimer();
	}
}


/**

*/
struct FOutputDeviceFile::FCategoryInclusionInternal
{
	TSet<FName> IncludedCategories;
};

/** 
 * Constructor, initializing member variables.
 *
 * @param InFilename		Filename to use, can be NULL
 * @param bInDisableBackup	If true, existing files will not be backed up
 */
FOutputDeviceFile::FOutputDeviceFile( const TCHAR* InFilename, bool bInDisableBackup, bool bInAppendIfExists, bool bCreateWriterLazily, TFunction<void(const TCHAR*)> FileOpenedCallback)
: AsyncWriter(nullptr)
, WriterArchive(nullptr)
, OnFileOpenedFn(MoveTemp(FileOpenedCallback))
, AppendIfExists(bInAppendIfExists)
, Dead(false)
, bCreatingWriter(false)
, bDisableBackup(bInDisableBackup)
, CategoryInclusionInternal(nullptr)
{
	if( InFilename )
	{
		FCString::Strncpy( Filename, InFilename, UE_ARRAY_COUNT(Filename) );
	}
	else
	{
		Filename[0]	= TCHAR('\0');
	}

#if ALLOW_LOG_FILE && !NO_LOGGING
	if (!bCreateWriterLazily) // Should the file created/opened immediately?
	{
		CreateWriter();
	}
#endif
}

/**
* Destructor to perform teardown
*
*/
FOutputDeviceFile::~FOutputDeviceFile()
{
	TearDown();
}

void FOutputDeviceFile::SetFilename(const TCHAR* InFilename)
{
	// Close any existing file.
	TearDown();

	FCString::Strncpy( Filename, InFilename, UE_ARRAY_COUNT(Filename) );
}

/**
 * Closes output device and cleans up. This can't happen in the destructor
 * as we have to call "delete" which cannot be done for static/ global
 * objects.
 */
void FOutputDeviceFile::TearDown()
{
	if (AsyncWriter)
	{
		if (!bSuppressEventTag)
		{			
			Logf(TEXT("Log file closed, %s"), FPlatformTime::StrTimestamp());
		}
		delete AsyncWriter;
		AsyncWriter = nullptr;
	}
	delete WriterArchive;
	WriterArchive = nullptr;

	Filename[0] = TCHAR('\0');
}

/**
 * Flush the write cache so the file isn't truncated in case we crash right
 * after calling this function.
 */
void FOutputDeviceFile::Flush()
{
	if (AsyncWriter)
	{
		AsyncWriter->Flush();
	}
}

/** if the passed in file exists, makes a timestamped backup copy
 * @param Filename the name of the file to check
 */
void FOutputDeviceFile::CreateBackupCopy(const TCHAR* Filename)
{
	IFileManager& FileManager = IFileManager::Get();
	if (FileManager.FileSize(Filename) > 0) // file exists and is not empty
	{
		FString Name, Extension;
		FString(Filename).Split(TEXT("."), &Name, &Extension, ESearchCase::CaseSensitive, ESearchDir::FromEnd);
		FDateTime OriginalTime = FileManager.GetTimeStamp(Filename);
		FString BackupFilename = FString::Printf(TEXT("%s%s%s.%s"), *Name, BACKUP_LOG_FILENAME_POSTFIX, *OriginalTime.ToString(), *Extension);
		if (FileManager.Copy(*BackupFilename, Filename, false) == COPY_OK)
		{
			FileManager.SetTimeStamp(*BackupFilename, OriginalTime);
		}
		// We use Copy + SetTimeStamp instead of Move because caller might want to append to original log file.
	}
}

bool FOutputDeviceFile::IsBackupCopy(const TCHAR* Filename)
{
	return Filename != nullptr && FCString::Stristr(const_cast<TCHAR*>(Filename), BACKUP_LOG_FILENAME_POSTFIX) != nullptr;
}

void FOutputDeviceFile::WriteByteOrderMarkToArchive(EByteOrderMark ByteOrderMark)
{
	switch (ByteOrderMark)
	{
	case EByteOrderMark::UTF8:
		AsyncWriter->Serialize(UTF8BOM, sizeof(UTF8BOM));
		break;

	case EByteOrderMark::Unspecified:
	default:
		check(false);
		break;
	}
}

bool FOutputDeviceFile::IsOpened() const
{
	return AsyncWriter != nullptr;
}

bool FOutputDeviceFile::CreateWriter(uint32 MaxAttempts)
{
	if (IsOpened())
	{
		return true;
	}

	// Ignore re-entrant calls to CreateWriter because that will create and leak a writer.
	if (bCreatingWriter)
	{
		return true;
	}
	TGuardValue<bool> SerializingScope(bCreatingWriter, true);

	// Make log filename.
	if (!Filename[0])
	{
		FCString::Strcpy(Filename, *FPlatformOutputDevices::GetAbsoluteLogFilename());
	}

	// if the file already exists, create a backup as we are going to overwrite it
	if (!bDisableBackup)
	{
		CreateBackupCopy(Filename);
	}

	// Create a silent filewriter so that it doesn't try to log any errors since it would redirect logging back to itself through this output device
	uint32 WriteFlags = FILEWRITE_Silent | FILEWRITE_AllowRead | (AppendIfExists ? FILEWRITE_Append : 0);

	// Open log file.
	FArchive* Ar = IFileManager::Get().CreateFileWriter(Filename, WriteFlags);

	// If that failed, append an _2 and try again (unless we don't want extra copies). This 
	// happens in the case of running a server and client on same computer for example.
	if (!bDisableBackup && !Ar)
	{
		FString FilenamePart = FPaths::GetBaseFilename(Filename, false) + "_";
		FString ExtensionPart = FPaths::GetExtension(Filename, true);
		FString FinalFilename;
		uint32 FileIndex = 2;
		do
		{
			// Continue to increment indices until a valid filename is found
			FinalFilename = FilenamePart + FString::FromInt(FileIndex++) + ExtensionPart;
			CreateBackupCopy(*FinalFilename);
			FCString::Strncpy(Filename, *FinalFilename, UE_ARRAY_COUNT(Filename));
			Ar = IFileManager::Get().CreateFileWriter(*FinalFilename, WriteFlags);
		} while (!Ar && FileIndex < MaxAttempts);
	}

	if (Ar)
	{
		const FAsyncWriter::EThreadNameOption NameOption = OUTPUTDEVICE_DEFAULT_ASYNC_WRITER_THREAD_NAME_OPTION;

		WriterArchive = Ar;
		AsyncWriter = new FAsyncWriter(*WriterArchive, NameOption);
		WriteByteOrderMarkToArchive(EByteOrderMark::UTF8);
		if (OnFileOpenedFn)
		{
			OnFileOpenedFn(Filename);
		}

		if (!bSuppressEventTag)
		{
			Logf(TEXT("Log file open, %s"), FPlatformTime::StrTimestamp());
		}

		IFileManager::Get().SetTimeStamp(Filename, FDateTime::UtcNow());

		return true;
	}
	else
	{
		return false;
	}
}

/**
 * Serializes the passed in data unless the current event is suppressed.
 *
 * @param	Data	Text to log
 * @param	Event	Event name used for suppression purposes
 */
void FOutputDeviceFile::Serialize( const TCHAR* Data, ELogVerbosity::Type Verbosity, const class FName& Category, const double Time )
{
#if ALLOW_LOG_FILE && !NO_LOGGING
	if (CategoryInclusionInternal && !CategoryInclusionInternal->IncludedCategories.Contains(Category))
	{
		return;
	}

	static bool Entry = false;
	if( !GIsCriticalError || Entry )
	{
		if (!AsyncWriter && !Dead)
		{
			// Open log file and create the worker thread.
			if (!CreateWriter())
			{
				Dead = true;
			}
		}

		if (AsyncWriter && Verbosity != ELogVerbosity::SetColor)
		{
			FOutputDeviceHelper::FormatCastAndSerializeLine(*AsyncWriter, Data, Verbosity, Category, Time, bSuppressEventTag, bAutoEmitLineTerminator);

			static bool GForceLogFlush = false;
			static bool GTestedCmdLine = false;
			if (!GTestedCmdLine)
			{
				GTestedCmdLine = true;
				// Force a log flush after each line
				GForceLogFlush = FParse::Param( FCommandLine::Get(), TEXT("FORCELOGFLUSH") );
			}
			if (GForceLogFlush)
			{
				AsyncWriter->Flush();
			}
		}
	}
	else
	{
		Entry = true;
		Serialize(Data, Verbosity, Category, Time);
		Entry = false;
	}
#endif
}

void FOutputDeviceFile::Serialize( const TCHAR* Data, ELogVerbosity::Type Verbosity, const class FName& Category )
{
	Serialize(Data, Verbosity, Category, -1.0);
}


void FOutputDeviceFile::WriteRaw( const TCHAR* C )
{
	AsyncWriter->Serialize((uint8*)const_cast<TCHAR*>(C), FCString::Strlen(C)*sizeof(TCHAR));
}

void FOutputDeviceFile::IncludeCategory(const FName& InCategoryName)
{
	if (!CategoryInclusionInternal.IsValid())
	{
		CategoryInclusionInternal = MakeUnique<FCategoryInclusionInternal>();
	}

	CategoryInclusionInternal->IncludedCategories.Add(InCategoryName);
}
