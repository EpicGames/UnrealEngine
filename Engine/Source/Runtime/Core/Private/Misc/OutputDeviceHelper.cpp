// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/OutputDeviceHelper.h"
#include "CoreGlobals.h"
#include "HAL/PlatformTime.h"
#include "Misc/App.h"
#include "Misc/DateTime.h"
#include "Misc/StringBuilder.h"

void FOutputDeviceHelper::AppendFormatLogLine(
	FWideStringBuilderBase& Format,
	const ELogVerbosity::Type Verbosity,
	const FName& Category,
	const TCHAR* const Message,
	const ELogTimes::Type LogTime,
	const double Time,
	int32* const OutCategoryIndex)
{
	switch (LogTime)
	{
		case ELogTimes::SinceGStartTime:
		{
			const double RealTime = (Time < 0.0) ? (FPlatformTime::Seconds() - GStartTime) : Time;
			Format.Appendf(WIDETEXT("[%07.2f][%3llu]"), RealTime, GFrameCounter % 1000);
			break;
		}

		case ELogTimes::UTC:
			FDateTime::UtcNow().ToString(TEXT("[%Y.%m.%d-%H.%M.%S:%s]"), Format);
			Format.Appendf(WIDETEXT("[%3llu]"), GFrameCounter % 1000);
			break;

		case ELogTimes::Local:
			FDateTime::Now().ToString(TEXT("[%Y.%m.%d-%H.%M.%S:%s]"), Format);
			Format.Appendf(WIDETEXT("[%3llu]"), GFrameCounter % 1000);
			break;

		case ELogTimes::Timecode:
			Format.Appendf(WIDETEXT("[%s][%3llu]"), *FApp::GetTimecode().ToString(), GFrameCounter % 1000);
			break;

		default:
			break;
	}

	const bool bShowCategory = GPrintLogCategory && !Category.IsNone();

	if (OutCategoryIndex)
	{
		*OutCategoryIndex = bShowCategory ? Format.Len() : INDEX_NONE;
	}

	if (bShowCategory)
	{
		Category.AppendString(Format);
		Format.Append(WIDETEXT(": "));

		if (GPrintLogVerbosity && Verbosity != ELogVerbosity::Log)
		{
			Format.Append(ToString(Verbosity));
			Format.Append(WIDETEXT(": "));
		}
	}
	else if (GPrintLogVerbosity && Verbosity != ELogVerbosity::Log)
	{
		Format.Append(ToString(Verbosity));
		Format.Append(WIDETEXT(": "));
	}

	if (Message)
	{
		Format.Append(Message);
	}
}

void FOutputDeviceHelper::AppendFormatLogLine(
	FUtf8StringBuilderBase& Format,
	const ELogVerbosity::Type Verbosity,
	const FName& Category,
	const TCHAR* const Message,
	const ELogTimes::Type LogTime,
	const double Time,
	int32* const OutCategoryIndex)
{
	TWideStringBuilder<128> Prefix;
	AppendFormatLogLine(Prefix, Verbosity, Category, nullptr, LogTime, Time, OutCategoryIndex);
	Format.Append(Prefix);

	if (Message)
	{
		Format.Append(Message);
	}
}

FString FOutputDeviceHelper::FormatLogLine(
	const ELogVerbosity::Type Verbosity,
	const FName& Category,
	const TCHAR* const Message,
	const ELogTimes::Type LogTime,
	const double Time,
	int32* const OutCategoryIndex)
{
	TStringBuilder<512> Format;
	AppendFormatLogLine(Format, Verbosity, Category, Message, LogTime, Time, OutCategoryIndex);
	return FString(Format);
}

void FOutputDeviceHelper::FormatCastAndSerializeLine(
	FArchive& Output,
	const TCHAR* const Message,
	const ELogVerbosity::Type Verbosity,
	const FName& Category,
	const double Time,
	const bool bSuppressEventTag,
	const bool bAutoEmitLineTerminator)
{
	TUtf8StringBuilder<512> Format;

	if (!bSuppressEventTag)
	{
		AppendFormatLogLine(Format, Verbosity, Category, Message, GPrintLogTimes, Time);
	}
	else
	{
		Format.Append(Message);
	}

	if (bAutoEmitLineTerminator)
	{
	#if PLATFORM_UNIX
		// Use Windows line endings on Linux for compatibility with Windows tools like notepad.exe
		Format.Append("\r\n");
	#else
		Format.Append(LINE_TERMINATOR_ANSI);
	#endif
	}

	Output.Serialize(Format.GetData(), Format.Len() * sizeof(UTF8CHAR));
}
