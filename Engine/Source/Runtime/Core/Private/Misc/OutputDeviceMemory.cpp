// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	OutputDeviceMemory.cpp: Ring buffer (memory only) output device
=============================================================================*/

#include "Misc/OutputDeviceMemory.h"

#include "CoreGlobals.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformOutputDevices.h"
#include "HAL/PlatformTime.h"
#include "Misc/CommandLine.h"
#include "Misc/OutputDeviceFile.h"
#include "Misc/OutputDeviceHelper.h"
#include "Misc/ScopeLock.h"
#include "Misc/Parse.h"

#define DUMP_LOG_ON_EXIT (!NO_LOGGING && PLATFORM_DESKTOP && (!UE_BUILD_SHIPPING || USE_LOGGING_IN_SHIPPING))

FOutputDeviceMemory::FOutputDeviceMemory(int32 InPreserveSize /*= 256 * 1024*/, int32 InBufferSize /*= 2048 * 1024*/, bool bInSuppressEventTag /*= false */)
: ArchiveProxy(*this)
, BufferStartPos(0)
, BufferLength(0)
, PreserveSize(InPreserveSize)
{
	bSuppressEventTag = bInSuppressEventTag;
#if DUMP_LOG_ON_EXIT
	const FString LogFileName = FPlatformOutputDevices::GetAbsoluteLogFilename();
	FOutputDeviceFile::CreateBackupCopy(*LogFileName);
	IFileManager::Get().Delete(*LogFileName);
#endif // DUMP_LOG_ON_EXIT

	checkf(InBufferSize >= InPreserveSize * 2, TEXT("FOutputDeviceMemory buffer size should be >= 2x PreserveSize"));

	Buffer.AddUninitialized(InBufferSize);
	if (!bSuppressEventTag)
	{
		Logf(TEXT("Log file open, %s"), FPlatformTime::StrTimestamp());
	}
}

FOutputDeviceMemory::~FOutputDeviceMemory() = default;

void FOutputDeviceMemory::TearDown() 
{
	if (!bSuppressEventTag)
	{
		Logf(TEXT("Log file closed, %s"), FPlatformTime::StrTimestamp());
	}
	// Dump on exit
#if DUMP_LOG_ON_EXIT
	const bool bDumpLogOnCrashOnly = FParse::Param(FCommandLine::Get(), TEXT("DumpLogOnExitCrashOnly"));
	if (GIsCriticalError || !bDumpLogOnCrashOnly)
	{
		const FString LogFileName = FPlatformOutputDevices::GetAbsoluteLogFilename();
		FArchive* LogFile = IFileManager::Get().CreateFileWriter(*LogFileName, FILEWRITE_AllowRead);
		if (LogFile)
		{
			Dump(*LogFile);
			LogFile->Flush();
			delete LogFile;
		}
	}
#endif // DUMP_LOG_ON_EXIT
}

void FOutputDeviceMemory::Flush()
{
	// Do nothing, buffer is always flushed
}

void FOutputDeviceMemory::Serialize(const TCHAR* Data, ELogVerbosity::Type Verbosity, const class FName& Category, const double Time)
{
#if !NO_LOGGING
	FOutputDeviceHelper::FormatCastAndSerializeLine(ArchiveProxy, Data, Verbosity, Category, Time, bSuppressEventTag, bAutoEmitLineTerminator);
#endif
}

void FOutputDeviceMemory::Serialize(const TCHAR* Data, ELogVerbosity::Type Verbosity, const class FName& Category)
{
	Serialize(Data, Verbosity, Category, -1.0);
}

void FOutputDeviceMemory::SerializeToBuffer(ANSICHAR* Data, int32 Length)
{	
	const int32 BufferCapacity = Buffer.Num(); // Never changes

	// Given the size of the buffer (usually 1MB) this should never happen
	ensure(Length <= BufferCapacity);

	while (Length)
	{
		int32 WritePos = 0;
		int32 WriteLength = 0;

		{
			// We only need to lock long enough to update all state variables. Copy doesn't need a lock, at least for large buffers.
			FScopeLock WriteLock(&BufferPosCritical);

			WritePos = BufferStartPos;

			// If this will take us past the buffer, wrap but preserve the specified amount of data at the head
			if (BufferStartPos + Length > BufferCapacity)
			{
				WriteLength = BufferCapacity - BufferStartPos;
				BufferStartPos = PreserveSize;
			}
			else
			{
				WriteLength = Length;
				BufferStartPos += WriteLength;
			}

			BufferLength = FMath::Min(BufferLength + WriteLength, BufferCapacity);
		}

		FMemory::Memcpy(Buffer.GetData() + WritePos, Data, WriteLength * sizeof(ANSICHAR));
		Data += WriteLength;
		Length -= WriteLength;
	}
}

void FOutputDeviceMemory::Dump(FArchive& Ar)
{
	const int32 BufferCapacity = Buffer.Num(); // Never changes

	// Dump the startup logs
	Ar.Serialize(Buffer.GetData(), FMath::Min(PreserveSize, BufferLength) * sizeof(ANSICHAR));

	// If the log has wrapped, dump the earliest portion chronologically from the end
	if (BufferLength == BufferCapacity)
	{
		ANSICHAR* BufferStartPosPtr = Buffer.GetData() + BufferStartPos * sizeof(ANSICHAR);
		const int32 PreWrapLength = BufferCapacity - BufferStartPos;
		Ar.Serialize(BufferStartPosPtr, PreWrapLength * sizeof(ANSICHAR));
	}

	// Dump the logs from preserved section up to current, if we've made it that far
	if (BufferLength > PreserveSize)
	{
		ANSICHAR* PreservePosPtr = Buffer.GetData() + PreserveSize * sizeof(ANSICHAR);
		const int32 AfterPreserveLength = BufferStartPos - PreserveSize;
		Ar.Serialize(PreservePosPtr, AfterPreserveLength * sizeof(ANSICHAR));
	}
}

bool FOutputDeviceMemory::DeserializeToBuffer(TArrayView<ANSICHAR> OutBuffer, int32 InOffsetInBytes) const
{
	const int32 PreservedSectionSize = FMath::Min(PreserveSize, BufferLength);  
	const int32 BufferCapacity = Buffer.Num();

	ANSICHAR* DataDest = OutBuffer.GetData();
	int32 SizeToCopy = OutBuffer.Num();
	
	int32 BytesCopied = 0;                                                                                                                                                                                                                                                                                                                                                             
	int32 BytesToSkip = InOffsetInBytes;

	// copy the startup logs
	if (BytesToSkip < PreservedSectionSize)
	{
		int32 ChunkSizeToCopy = FMath::Min(PreservedSectionSize - BytesToSkip, SizeToCopy);
		const ANSICHAR* PreservedSectionPosPtr = Buffer.GetData() + BytesToSkip * sizeof(ANSICHAR);
		FMemory::Memcpy(DataDest, PreservedSectionPosPtr, ChunkSizeToCopy);
		BytesCopied += ChunkSizeToCopy;
		DataDest += ChunkSizeToCopy;
		BytesToSkip = 0;
	}
	else
	{
		BytesToSkip = FMath::Max(BytesToSkip - PreservedSectionSize, 0);
	}

	// If the log has wrapped, copy the earliest portion chronologically from the end
	if (BufferLength == BufferCapacity && BytesCopied < SizeToCopy)
	{
		const int32 PreWrapLength = BufferCapacity - BufferStartPos;
		if (BytesToSkip < PreWrapLength)
		{
			int32 ChunkSizeToCopy = FMath::Min(PreWrapLength - BytesToSkip, SizeToCopy - BytesCopied);
			const ANSICHAR* BufferStartPosPtr = Buffer.GetData() + BufferStartPos + BytesToSkip * sizeof(ANSICHAR);
			FMemory::Memcpy(DataDest, BufferStartPosPtr, ChunkSizeToCopy);
			BytesCopied += ChunkSizeToCopy;
			DataDest += ChunkSizeToCopy;
			BytesToSkip = 0;
		}
		else
		{
			BytesToSkip = FMath::Max(BytesToSkip - PreWrapLength, 0);
		}
	}

	// Dump the logs from preserved section up to current, if we've made it that far
	if (BufferLength > PreserveSize && BytesCopied < SizeToCopy && BytesToSkip < BufferStartPos - PreserveSize)
	{
		int32 ChunkSizeToCopy = FMath::Min(BufferStartPos - PreserveSize - BytesToSkip, SizeToCopy - BytesCopied);
		const ANSICHAR* PreservePosPtr = Buffer.GetData() + PreserveSize + BytesToSkip * sizeof(ANSICHAR);
		FMemory::Memcpy(DataDest, PreservePosPtr, ChunkSizeToCopy);
		BytesCopied += ChunkSizeToCopy;
		DataDest += ChunkSizeToCopy;
	}

	return BytesCopied != 0;
}