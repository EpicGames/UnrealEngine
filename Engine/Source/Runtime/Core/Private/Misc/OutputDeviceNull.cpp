// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/OutputDeviceNull.h"

void FOutputDeviceNull::Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const class FName& Category)
{
}

