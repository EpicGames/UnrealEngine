// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/OutputDeviceRedirector.h"

#include "Async/EventCount.h"
#include "AutoRTFM.h"
#include "Containers/BitArray.h"
#include "Containers/MpscQueue.h"
#include "Experimental/ConcurrentLinearAllocator.h"
#include "HAL/PlatformProcess.h"
#include "HAL/PlatformTime.h"
#include "HAL/PlatformTLS.h"
#include "HAL/Thread.h"
#include "Logging/StructuredLog.h"
#include "Misc/App.h"
#include "Misc/ScopeRWLock.h"
#include "Misc/StringBuilder.h"
#include "Misc/TransactionallySafeRWLock.h"
#include "Misc/TVariant.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include <atomic>

/*-----------------------------------------------------------------------------
	FOutputDeviceRedirector.
-----------------------------------------------------------------------------*/

FBufferedLine::FBufferedLine(const TCHAR* InData, const FName& InCategory, ELogVerbosity::Type InVerbosity, double InTime)
	: Category(InCategory)
	, Time(InTime)
	, Verbosity(InVerbosity)
{
	int32 NumChars = FCString::Strlen(InData) + 1;
	Data = MakeUniqueForOverwrite<TCHAR[]>(sizeof(TCHAR) * NumChars);
	FMemory::Memcpy(Data.Get(), InData, sizeof(TCHAR) * NumChars);
}

namespace UE::Private
{

struct FOutputDeviceBlockAllocationTag : FDefaultBlockAllocationTag
{
	static constexpr const char* TagName = "OutputDeviceLinear";

	using Allocator = FAlignedAllocator;
};

struct FOutputDeviceLinearAllocator
{
	FORCEINLINE static void* Malloc(SIZE_T Size, uint32 Alignment)
	{
		return TConcurrentLinearAllocator<FOutputDeviceBlockAllocationTag>::Malloc(Size, Alignment);
	}

	FORCEINLINE static void Free(void* Pointer)
	{
		TConcurrentLinearAllocator<FOutputDeviceBlockAllocationTag>::Free(Pointer);
	}
};

struct FOutputDeviceLine
{
	const double Time;
	const TCHAR* Data;
	const FName Category;
	const ELogVerbosity::Type Verbosity;

	FOutputDeviceLine(const FOutputDeviceLine&) = delete;
	FOutputDeviceLine& operator=(const FOutputDeviceLine&) = delete;

	inline FOutputDeviceLine(FOutputDeviceLine&& Other)
		: Time(Other.Time)
		, Data(Other.Data)
		, Category(Other.Category)
		, Verbosity(Other.Verbosity)
	{
		Other.Data = nullptr;
	}

	inline FOutputDeviceLine(const TCHAR* const InData, const FName& InCategory, const ELogVerbosity::Type InVerbosity, const double InTime)
		: Time(InTime)
		, Data(CopyData(InData))
		, Category(InCategory)
		, Verbosity(InVerbosity)
	{
	}

	inline ~FOutputDeviceLine()
	{
		if (Data)
		{
			FOutputDeviceLinearAllocator::Free(const_cast<TCHAR*>(Data));
		}
	}

private:
	inline static const TCHAR* CopyData(const TCHAR* const InData)
	{
		const int32 Len = FCString::Strlen(InData) + 1;
		void* const Dest = FOutputDeviceLinearAllocator::Malloc(sizeof(TCHAR) * Len, alignof(TCHAR));
		return static_cast<TCHAR*>(FMemory::Memcpy(Dest, InData, sizeof(TCHAR) * Len));
	}
};

struct FOutputDeviceItem
{
	TVariant<FOutputDeviceLine, FLogRecord, uint64> Value;

	inline FOutputDeviceItem(const TCHAR* const Data, const FName& Category, const ELogVerbosity::Type Verbosity, const double Time)
		: Value(TInPlaceType<FOutputDeviceLine>(), Data, Category, Verbosity, Time)
	{
	}

	inline explicit FOutputDeviceItem(const FLogRecord& Record)
		: Value(TInPlaceType<FLogRecord>(), Record)
	{
	}

	inline explicit FOutputDeviceItem(uint64 FenceIndex)
		: Value(TInPlaceType<uint64>(), FenceIndex)
	{
	}

	FOutputDeviceItem(FOutputDeviceItem&&) = default;
	FOutputDeviceItem& operator=(FOutputDeviceItem&&) = default;

	FOutputDeviceItem(const FOutputDeviceItem&) = delete;
	FOutputDeviceItem& operator=(const FOutputDeviceItem&) = delete;
};

static constexpr uint64 CalculateRedirectorCacheLinePadding(const uint64 Size)
{
	return PLATFORM_CACHE_LINE_SIZE * FMath::DivideAndRoundUp<uint64>(Size, PLATFORM_CACHE_LINE_SIZE) - Size;
}

struct FOutputDeviceRedirectorState
{
	/** A custom lock to guard access to both buffered and unbuffered output devices. */
	FRWLock OutputDevicesLock;
	std::atomic<uint32> OutputDevicesLockState = 0;
	uint8 OutputDevicesLockPadding[CalculateRedirectorCacheLinePadding(sizeof(OutputDevicesLock) + sizeof(OutputDevicesLockState))]{};

	/** A queue of items logged by non-primary threads. */
	TMpscQueue<FOutputDeviceItem, FOutputDeviceLinearAllocator> BufferedItems;
	std::atomic<uint32> BufferedItemCount = 0;

	/** Array of output devices to redirect to from the primary thread. */
	TArray<FOutputDevice*> BufferedOutputDevices;

	/** Array of output devices to redirect to from the calling thread. */
	TArray<FOutputDevice*> UnbufferedOutputDevices;

	/** A queue of lines logged before the editor added its output device. */
	TArray<FBufferedLine> BacklogLines;
	FRWLock BacklogLock;
	std::atomic<uint32> SerializeBacklogCount = 0;

	/** An optional dedicated primary thread for logging to buffered output devices. */
	FThread Thread;

	/** A lock to synchronize access to the thread. */
	FRWLock ThreadLock;

	/** An event that is notified when the dedicated primary thread is idle. */
	FEventCount ThreadIdleEvent;

	/** An event to wake the dedicated primary thread to process buffered items. */
	FEventCount ThreadWakeEvent;

	/** An event to wake threads waiting on fences. */
	FEventCount FenceCompleteEvent;

	/** The index of the last created fence. */
	std::atomic<uint64> FenceCreateIndex = 0;

	/** The index of the last completed fence. Never decreases even when fences complete out of order. */
	std::atomic<uint64> FenceCompleteIndex = 0;

	/** The ID of the thread holding the primary lock. */
	std::atomic<uint32> LockedThreadId = MAX_uint32;

	/** The ID of the primary logging thread. Logging from other threads will be buffered for processing by the primary thread. */
	std::atomic<uint32> PrimaryThreadId = FPlatformTLS::GetCurrentThreadId();

	/** The ID of the panic thread, which is only set by Panic(). */
	std::atomic<uint32> PanicThreadId = MAX_uint32;

	/** Whether a dedicated primary thread has been started. */
	std::atomic<bool> bThreadStarted = false;

	/** Whether the backlog is enabled. */
	bool bEnableBacklog = !IS_PROGRAM; // Programs may not use FEngineLoop which disables the backlog but other targets do.

	/** Whether the output device at the corresponding index can be used on the panic thread. */
	TBitArray<TInlineAllocator<1>> BufferedOutputDevicesCanBeUsedOnPanicThread;
	TBitArray<TInlineAllocator<1>> UnbufferedOutputDevicesCanBeUsedOnPanicThread;

	bool HasPanicThread() const
	{
		return PanicThreadId.load(std::memory_order_relaxed) != MAX_uint32;
	}

	bool IsPrimaryThread(const uint32 ThreadId) const
	{
		return ThreadId == PrimaryThreadId.load(std::memory_order_relaxed);
	}

	bool IsPanicThread(const uint32 ThreadId) const
	{
		return ThreadId == PanicThreadId.load(std::memory_order_relaxed);
	}

	bool CanLockFromThread(const uint32 ThreadId) const
	{
		if (UNLIKELY(ThreadId == LockedThreadId.load(std::memory_order_relaxed)))
		{
			return false;
		}
		const uint32 LocalPanicThreadId = PanicThreadId.load(std::memory_order_relaxed);
		return LocalPanicThreadId == MAX_uint32 || LocalPanicThreadId == ThreadId;
	}

	void AddOutputDevice(FOutputDevice* OutputDevice);
	void RemoveOutputDevice(FOutputDevice* OutputDevice);

	bool TryStartThread();
	bool TryStopThread();

	void ThreadLoop();

	void ConditionalFlushBufferedItems(EOutputDeviceRedirectorFlushOptions Options);
	void FlushBufferedItems();

	template <typename VisitorType>
	void FlushBufferedItemsTo(VisitorType&& Visitor);

	uint64 CreateFence();
	void WaitFence(uint64 Index);

	template <typename OutputDevicesType, typename FunctionType, typename... ArgTypes>
	FORCEINLINE void BroadcastTo(
		const uint32 ThreadId,
		const OutputDevicesType& OutputDevices,
		const TBitArray<TInlineAllocator<1>>& CanBeUsedOnPanicThread,
		FunctionType&& Function,
		ArgTypes&&... Args)
	{
		int32 Index = 0;
		const bool bIsPanicThread = IsPanicThread(ThreadId);
		for (FOutputDevice* OutputDevice : OutputDevices)
		{
			if (!bIsPanicThread || CanBeUsedOnPanicThread[Index++])
			{
				Invoke(Function, OutputDevice, Forward<ArgTypes>(Args)...);
			}
		}
	}

	FORCENOINLINE bool EnableBacklog(bool bEnable);
	FORCENOINLINE void AddToBacklog(const FLogRecord& Record);
	FORCENOINLINE void AddToBacklog(const TCHAR* Data, ELogVerbosity::Type Verbosity, const FName& Category, double Time);
	FORCENOINLINE void SerializeBacklog(TNotNull<FOutputDevice*> OutputDevice);
};

/**
 * A scoped lock for readers of the OutputDevices arrays.
 *
 * The read lock:
 * - Must be locked to read the OutputDevices arrays.
 * - Must be locked to write to unbuffered output devices.
 * - Must not be entered when the thread holds a write or primary lock.
 */
class FOutputDevicesReadScopeLock
{
public:
	FORCEINLINE explicit FOutputDevicesReadScopeLock(FOutputDeviceRedirectorState& InState)
		: State(InState)
	{
		AcquireLock();
	}

	FORCEINLINE ~FOutputDevicesReadScopeLock()
	{
		ReleaseLock();
	}

private:
	void AcquireLock()
	{
		// Read locks add/sub by 2 to keep the LSB free for write locks to use.
		if (State.OutputDevicesLockState.fetch_add(2, std::memory_order_acquire) & 1)
		{
			WaitForWriteLock();
		}
	}

	void ReleaseLock()
	{
		State.OutputDevicesLockState.fetch_sub(2, std::memory_order_release);
	}

	FORCENOINLINE void WaitForWriteLock()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FOutputDevicesReadScopeLock);
		// A write lock has set the LSB. Cancel this read lock and wait for the write.
		State.OutputDevicesLockState.fetch_sub(2, std::memory_order_relaxed);
		// This read lock will wait until the write lock exits.
		TReadScopeLock ScopeLock(State.OutputDevicesLock);
		// This is relaxed because locking OutputDevicesLock has acquire semantics.
		uint32 LockState = State.OutputDevicesLockState.fetch_add(2, std::memory_order_relaxed);
		check((LockState & 1) == 0);
	}

	FOutputDeviceRedirectorState& State;
};

/**
 * A scoped lock for writers of the OutputDevices arrays.
 *
 * The write lock has the same access as the primary lock, and:
 * - Must be locked to add or remove output devices.
 * - Must not be entered when the thread holds a read, write, or primary lock.
 */
class FOutputDevicesWriteScopeLock
{
public:
	FORCEINLINE explicit FOutputDevicesWriteScopeLock(FOutputDeviceRedirectorState& InState)
		: State(InState)
	{
		// FOutputDevicesWriteScopeLock is not transactionally safe.
		check(!AutoRTFM::IsClosed());

		TRACE_CPUPROFILER_EVENT_SCOPE(FOutputDevicesWriteScopeLock);
		// Take the lock before modifying the state, to avoid contention on the LSB.
		State.OutputDevicesLock.WriteLock();
		State.LockedThreadId.store(FPlatformTLS::GetCurrentThreadId(), std::memory_order_relaxed);
		// Set the LSB to flag to read locks that a write lock is waiting.
		uint32 LockState = State.OutputDevicesLockState.fetch_or(uint32(1), std::memory_order_acquire);
		check((LockState & 1) == 0);
		if (LockState > 1)
		{
			// Wait for read locks to be cleared.
			do
			{
				FPlatformProcess::Sleep(0);
				LockState = State.OutputDevicesLockState.load(std::memory_order_acquire);
			}
			while (LockState > 1);
		}
	}

	FORCEINLINE ~FOutputDevicesWriteScopeLock()
	{
		// Clear the LSB to allow read locks after the unlock below.
		uint32 LockState = State.OutputDevicesLockState.fetch_and(~uint32(1), std::memory_order_release);
		check((LockState & 1) == 1);
		State.LockedThreadId.store(MAX_uint32, std::memory_order_relaxed);
		State.OutputDevicesLock.WriteUnlock();
	}

private:
	FOutputDeviceRedirectorState& State;
};

/**
 * A scoped lock for exclusive access to the state of the primary log thread.
 *
 * The primary lock has the same access as the read lock, and:
 * - Must not be entered when the thread holds a write lock or primary lock.
 * - Must check IsLocked() before performing restricted operations.
 * - Must be locked to write to buffered output devices.
 * - Must be locked while calling FlushBufferedItems().
 * - May be locked when the thread holds a read lock.
 * - When a panic thread is active, locking will only succeed from the panic thread.
 */
class FOutputDevicesPrimaryScopeLock
{
public:
	explicit FOutputDevicesPrimaryScopeLock(FOutputDeviceRedirectorState& InState)
		: State(InState)
	{
		const uint32 ThreadId = FPlatformTLS::GetCurrentThreadId();
		if (State.CanLockFromThread(ThreadId))
		{
			if (State.IsPanicThread(ThreadId))
			{
				bLocked = true;
			}
			else
			{
				State.OutputDevicesLock.WriteLock();
				if (!State.CanLockFromThread(ThreadId))
				{
					State.OutputDevicesLock.WriteUnlock();
				}
				else
				{
					bNeedsUnlock = true;
					bLocked = true;
				}
			}
			if (bLocked)
			{
				State.LockedThreadId.store(ThreadId, std::memory_order_relaxed);
			}
		}
	}

	FORCEINLINE ~FOutputDevicesPrimaryScopeLock()
	{
		if (bLocked)
		{
			State.LockedThreadId.store(MAX_uint32, std::memory_order_relaxed);
		}
		if (bNeedsUnlock)
		{
			State.OutputDevicesLock.WriteUnlock();
		}
	}

	FORCEINLINE bool IsLocked() const { return bLocked; }

private:
	FOutputDeviceRedirectorState& State;
	bool bNeedsUnlock = false;
	bool bLocked = false;
};

UE_AUTORTFM_NOAUTORTFM
void FOutputDeviceRedirectorState::AddOutputDevice(FOutputDevice* OutputDevice)
{
	const auto AddTo = [OutputDevice](TArray<FOutputDevice*>& OutputDevices, TBitArray<TInlineAllocator<1>>& Flags)
	{
		const int32 Count = OutputDevices.Num();
		if (OutputDevices.AddUnique(OutputDevice) == Count)
		{
			Flags.Add(OutputDevice->CanBeUsedOnPanicThread());
		}
	};
	FOutputDevicesWriteScopeLock ScopeLock(*this);
	if (OutputDevice->CanBeUsedOnMultipleThreads())
	{
		AddTo(UnbufferedOutputDevices, UnbufferedOutputDevicesCanBeUsedOnPanicThread);
	}
	else
	{
		AddTo(BufferedOutputDevices, BufferedOutputDevicesCanBeUsedOnPanicThread);
	}
}

UE_AUTORTFM_NOAUTORTFM
void FOutputDeviceRedirectorState::RemoveOutputDevice(FOutputDevice* OutputDevice)
{
	const auto RemoveFrom = [OutputDevice](TArray<FOutputDevice*>& OutputDevices, TBitArray<TInlineAllocator<1>>& Flags)
	{
		if (const int32 Index = OutputDevices.FindLast(OutputDevice); Index != INDEX_NONE)
		{
			OutputDevices.RemoveAt(Index);
			Flags.RemoveAt(Index);
		}
	};
	FOutputDevicesWriteScopeLock ScopeLock(*this);
	RemoveFrom(BufferedOutputDevices, BufferedOutputDevicesCanBeUsedOnPanicThread);
	RemoveFrom(UnbufferedOutputDevices, UnbufferedOutputDevicesCanBeUsedOnPanicThread);
}

UE_AUTORTFM_NOAUTORTFM
bool FOutputDeviceRedirectorState::TryStartThread()
{
	if (FWriteScopeLock ThreadScopeLock(ThreadLock); !bThreadStarted.exchange(true, std::memory_order_relaxed))
	{
		Thread = FThread(TEXT("OutputDeviceRedirector"), [this] { ThreadLoop(); });
	}
	return true;
}

UE_AUTORTFM_NOAUTORTFM
bool FOutputDeviceRedirectorState::TryStopThread()
{
	if (FWriteScopeLock ThreadScopeLock(ThreadLock); bThreadStarted.exchange(false, std::memory_order_relaxed))
	{
		ThreadWakeEvent.Notify();
		Thread.Join();
	}
	return true;
}

UE_AUTORTFM_NOAUTORTFM
void FOutputDeviceRedirectorState::ThreadLoop()
{
	const uint32 ThreadId = FPlatformTLS::GetCurrentThreadId();

	if (FOutputDevicesPrimaryScopeLock Lock(*this); Lock.IsLocked())
	{
		PrimaryThreadId.store(ThreadId, std::memory_order_relaxed);
	}

	for (;;)
	{
		FEventCountToken Token = ThreadWakeEvent.PrepareWait();
		if (!bThreadStarted.load(std::memory_order_relaxed))
		{
			break;
		}
		while (BufferedItemCount.load(std::memory_order_relaxed) && IsPrimaryThread(ThreadId))
		{
			if (FOutputDevicesPrimaryScopeLock Lock(*this); Lock.IsLocked())
			{
				FlushBufferedItems();
			}
		}
		ThreadIdleEvent.Notify();
		ThreadWakeEvent.Wait(Token);
	}
}

UE_AUTORTFM_NOAUTORTFM
void FOutputDeviceRedirectorState::ConditionalFlushBufferedItems(EOutputDeviceRedirectorFlushOptions Options)
{
	if (FReadScopeLock Lock(ThreadLock); bThreadStarted.load(std::memory_order_relaxed))
	{
		if (!EnumHasAnyFlags(Options, EOutputDeviceRedirectorFlushOptions::Async))
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FOutputDeviceRedirector::FlushThreadedLogs);
			FEventCountToken Token = ThreadIdleEvent.PrepareWait();
			ThreadWakeEvent.Notify();
			ThreadIdleEvent.Wait(Token);
		}
		return;
	}

	if (FOutputDevicesPrimaryScopeLock Lock(*this); Lock.IsLocked())
	{
		FlushBufferedItems();
	}
}

UE_AUTORTFM_NOAUTORTFM
void FOutputDeviceRedirectorState::FlushBufferedItems()
{
	if (!BufferedItemCount.load(std::memory_order_relaxed))
	{
		return;
	}

	struct FVisitor
	{
		FOutputDeviceRedirectorState& State;
		const uint32 ThreadId = FPlatformTLS::GetCurrentThreadId();
		const bool bIsPanicThread = State.IsPanicThread(ThreadId);

		inline void operator()(const FOutputDeviceLine& Line)
		{
			State.BroadcastTo(ThreadId, State.BufferedOutputDevices, State.BufferedOutputDevicesCanBeUsedOnPanicThread,
				UE_PROJECTION_MEMBER(FOutputDevice, Serialize),
				Line.Data, Line.Verbosity, Line.Category, Line.Time);
		}

		inline void operator()(const FLogRecord& Record)
		{
			State.BroadcastTo(ThreadId, State.BufferedOutputDevices, State.BufferedOutputDevicesCanBeUsedOnPanicThread,
				UE_PROJECTION_MEMBER(FOutputDevice, SerializeRecord), Record);
		}
	};
	FlushBufferedItemsTo(FVisitor{*this});
}

template <typename VisitorType>
UE_AUTORTFM_NOAUTORTFM
void FOutputDeviceRedirectorState::FlushBufferedItemsTo(VisitorType&& Visitor)
{
	if (!BufferedItemCount.load(std::memory_order_relaxed))
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FOutputDeviceRedirector::FlushBufferedItems);

	for (uint32 Count = BufferedItemCount.exchange(0, std::memory_order_relaxed); Count; --Count)
	{
		for (;;)
		{
			if (!Visitor.bIsPanicThread && HasPanicThread())
			{
				// Re-queue items to be consumed by the panic thread.
				BufferedItemCount.fetch_add(Count, std::memory_order_relaxed);
				return;
			}
			
			// Dequeue can return NullOpt even though it's not empty due to races of setting Prev->Next	in Enqueue		
			if (TOptional<FOutputDeviceItem> Item = BufferedItems.Dequeue())
			{
				Visit([this, &Visitor]<class ValueType>(const ValueType& Value)
				{
					if constexpr (!std::is_same_v<ValueType, uint64>)
					{
						Visitor(Value);
					}
					else
					{
						const uint64 FenceIndex = Value;
						for (uint64 LastFenceIndex = FenceCompleteIndex.load(std::memory_order_relaxed);;)
						{
							if (FenceIndex <= LastFenceIndex || FenceCompleteIndex.compare_exchange_weak(LastFenceIndex, FenceIndex, std::memory_order_release))
							{
								break;
							}
						}
						FenceCompleteEvent.Notify();
					}
				}, Item->Value);
				break;
			}
			
			FPlatformProcess::Yield();
		}
	}
}

UE_AUTORTFM_NOAUTORTFM
uint64 FOutputDeviceRedirectorState::CreateFence()
{
	uint64 Index = FenceCreateIndex.fetch_add(1, std::memory_order_relaxed) + 1;
	BufferedItems.Enqueue(Index);
	if (!BufferedItemCount.fetch_add(1, std::memory_order_relaxed))
	{
		ThreadWakeEvent.Notify();
	}
	return Index;
}

UE_AUTORTFM_NOAUTORTFM
void FOutputDeviceRedirectorState::WaitFence(uint64 Index)
{
	for (;;)
	{
		if (Index <= FenceCompleteIndex.load(std::memory_order_acquire))
		{
			return;
		}

		ConditionalFlushBufferedItems(EOutputDeviceRedirectorFlushOptions::Async);

		FEventCountToken Token = FenceCompleteEvent.PrepareWait();
		if (Index <= FenceCompleteIndex.load(std::memory_order_acquire))
		{
			return;
		}
		FenceCompleteEvent.Wait(Token);
	}
}

UE_AUTORTFM_NOAUTORTFM
bool FOutputDeviceRedirectorState::EnableBacklog(bool bEnable)
{
	FWriteScopeLock ScopeLock(BacklogLock);
	const bool bWasEnabled = bEnableBacklog;
	bEnableBacklog = bEnable;
	if (!bEnable)
	{
		BacklogLines.Empty();
	}
	else if (!bWasEnabled)
	{
		// Move any buffered items into the backlog if there are no buffered output devices
		// registered yet. This can only occur in program targets. The backlog is a more robust
		// means of capturing early logging.
		const uint32 ThreadId = FPlatformTLS::GetCurrentThreadId();
		if (FOutputDevicesPrimaryScopeLock PrimaryLock(*this);
			PrimaryLock.IsLocked() && IsPrimaryThread(ThreadId) && BufferedOutputDevices.IsEmpty())
		{
			struct FVisitor
			{
				FOutputDeviceRedirectorState& State;
				const bool bIsPanicThread;

				inline void operator()(const FOutputDeviceLine& Line)
				{
					State.BacklogLines.Emplace(Line.Data, Line.Category, Line.Verbosity, Line.Time);
				}

				inline void operator()(const FLogRecord& Record)
				{
					TStringBuilder<512> Text;
					Record.FormatMessageTo(Text);
					State.BacklogLines.Emplace(*Text, Record.GetCategory(), Record.GetVerbosity(), FPlatformTime::Seconds() - GStartTime);
				}
			};
			FlushBufferedItemsTo(FVisitor{*this, IsPanicThread(ThreadId)});
		}
	}
	return bWasEnabled;
}

UE_AUTORTFM_NOAUTORTFM
void FOutputDeviceRedirectorState::AddToBacklog(const FLogRecord& Record)
{
	TStringBuilder<512> Text;
	Record.FormatMessageTo(Text);
	AddToBacklog(*Text, Record.GetVerbosity(), Record.GetCategory(), FPlatformTime::Seconds() - GStartTime);
}

UE_AUTORTFM_NOAUTORTFM
void FOutputDeviceRedirectorState::AddToBacklog(
	const TCHAR* const Data,
	const ELogVerbosity::Type Verbosity,
	const FName& Category,
	const double Time)
{
	// Skip adding to the backlog if it happens while serializing the backlog.
	// This is a workaround until the backlog storage is safe to write during iteration.
	if (SerializeBacklogCount.load(std::memory_order_relaxed) > 0)
	{
		return;
	}
	FWriteScopeLock ScopeLock(BacklogLock);
	BacklogLines.Emplace(Data, Category, Verbosity, Time);
}

void FOutputDeviceRedirectorState::SerializeBacklog(TNotNull<FOutputDevice*> OutputDevice)
{
	FReadScopeLock ScopeLock(BacklogLock);
	SerializeBacklogCount.fetch_add(1, std::memory_order_relaxed);
	for (const FBufferedLine& BacklogLine : BacklogLines)
	{
		OutputDevice->Serialize(BacklogLine.Data.Get(), BacklogLine.Verbosity, BacklogLine.Category, BacklogLine.Time);
	}
	SerializeBacklogCount.fetch_sub(1, std::memory_order_relaxed);
}

} // UE::Private

UE_AUTORTFM_NOAUTORTFM // FOutputDeviceRedirector cannot be constructed in a closed transaction
FOutputDeviceRedirector::FOutputDeviceRedirector()
	: State(MakePimpl<UE::Private::FOutputDeviceRedirectorState>())
{
}

static bool GWasGLogDestroyed = false;

UE_AUTORTFM_NOAUTORTFM // FOutputDeviceRedirector cannot be destructed in a closed transaction
FOutputDeviceRedirector::~FOutputDeviceRedirector()
{
	if (this == FOutputDeviceRedirector::Get())
	{
		GWasGLogDestroyed = true;
	}
}

FOutputDeviceRedirector* FOutputDeviceRedirector::Get()
{
	static FOutputDeviceRedirector Singleton;
	return &Singleton;
}

UE_AUTORTFM_NOAUTORTFM // Not currently supported in a closed transaction
void FOutputDeviceRedirector::AddOutputDevice(FOutputDevice* OutputDevice)
{
	if (OutputDevice)
	{
		if (State->bEnableBacklog)
		{
			// Flush buffered items before adding the output device to avoid an item being logged
			// twice due to being in both the backlog and the buffer.
			State->ConditionalFlushBufferedItems(EOutputDeviceRedirectorFlushOptions::None);
		}

		State->AddOutputDevice(OutputDevice);

		if (State->bEnableBacklog)
		{
			State->SerializeBacklog(OutputDevice);
		}
	}
}

UE_AUTORTFM_NOAUTORTFM // Not currently supported in a closed transaction
void FOutputDeviceRedirector::RemoveOutputDevice(FOutputDevice* OutputDevice)
{
	if (OutputDevice)
	{
		State->RemoveOutputDevice(OutputDevice);
	}
}

UE_AUTORTFM_ALWAYS_OPEN
bool FOutputDeviceRedirector::IsRedirectingTo(FOutputDevice* OutputDevice)
{
	UE::Private::FOutputDevicesReadScopeLock Lock(*State);
	return State->BufferedOutputDevices.Contains(OutputDevice) || State->UnbufferedOutputDevices.Contains(OutputDevice);
}

UE_AUTORTFM_ALWAYS_OPEN
void FOutputDeviceRedirector::FlushThreadedLogs(EOutputDeviceRedirectorFlushOptions Options)
{
	State->ConditionalFlushBufferedItems(Options);
}

UE_AUTORTFM_ALWAYS_OPEN
void FOutputDeviceRedirector::SerializeBacklog(FOutputDevice* OutputDevice)
{
	// Deprecated. Do nothing because AddOutputDevice now serializes the backlog.
}

UE_AUTORTFM_NOAUTORTFM // Not currently supported in a closed transaction
bool FOutputDeviceRedirector::EnableBacklog(bool bEnable)
{
	return State->EnableBacklog(bEnable);
}

UE_AUTORTFM_NOAUTORTFM // Not currently supported in a closed transaction
void FOutputDeviceRedirector::SetCurrentThreadAsPrimaryThread()
{
	const uint32 ThreadId = FPlatformTLS::GetCurrentThreadId();

	if (UE::Private::FOutputDevicesPrimaryScopeLock Lock(*State); !Lock.IsLocked() || State->PrimaryThreadId.load(std::memory_order_relaxed) == ThreadId)
	{
		return;
	}
	else
	{
		State->PrimaryThreadId.store(ThreadId, std::memory_order_relaxed);
		State->FlushBufferedItems();
	}

	State->TryStopThread();
}

UE_AUTORTFM_NOAUTORTFM // Not currently supported in a closed transaction
bool FOutputDeviceRedirector::TryStartDedicatedPrimaryThread()
{
	return FApp::ShouldUseThreadingForPerformance() && State->TryStartThread();
}

UE_AUTORTFM_ALWAYS_OPEN
void FOutputDeviceRedirector::SerializeRecord(const UE::FLogRecord& Record)
{
	using namespace UE::Private;

	FOutputDevicesReadScopeLock Lock(*State);

	const uint32 ThreadId = FPlatformTLS::GetCurrentThreadId();

	// Serialize directly to any output devices which don't require buffering
	State->BroadcastTo(ThreadId, State->UnbufferedOutputDevices, State->UnbufferedOutputDevicesCanBeUsedOnPanicThread,
		UE_PROJECTION_MEMBER(FOutputDevice, SerializeRecord), Record);

	// Serialize to the backlog when not in panic mode. This will deadlock in panic mode when the
	// FPlatformMallocCrash allocator has been enabled and logging occurs on a non-panic thread.
	if (UNLIKELY(State->bEnableBacklog && !State->HasPanicThread()))
	{
		State->AddToBacklog(Record);
	}

	// Serialize to buffered output devices from the primary logging thread.
	// Records are queued until buffered output devices are added to avoid missing early log records.
	if (State->IsPrimaryThread(ThreadId) && !State->BufferedOutputDevices.IsEmpty())
	{
		// Verify that this is the primary thread again because another thread may have become
		// the primary thread between the previous check and the lock.
		if (FOutputDevicesPrimaryScopeLock PrimaryLock(*State); PrimaryLock.IsLocked() && State->IsPrimaryThread(ThreadId))
		{
			State->FlushBufferedItems();
			State->BroadcastTo(ThreadId, State->BufferedOutputDevices, State->BufferedOutputDevicesCanBeUsedOnPanicThread,
				UE_PROJECTION_MEMBER(FOutputDevice, SerializeRecord), Record);
			if (UNLIKELY(State->IsPanicThread(ThreadId)))
			{
				Flush();
			}
			return;
		}
	}

	// Queue the record to serialize to buffered output devices from the primary thread.
	State->BufferedItems.Enqueue(Record);
	if (!State->BufferedItemCount.fetch_add(1, std::memory_order_relaxed))
	{
		State->ThreadWakeEvent.Notify();
	}
}

UE_AUTORTFM_ALWAYS_OPEN
void FOutputDeviceRedirector::Serialize(const TCHAR* const Data, const ELogVerbosity::Type Verbosity, const FName& Category, const double Time)
{
	using namespace UE::Private;

	const double RealTime = Time == -1.0 ? FPlatformTime::Seconds() - GStartTime : Time;

	FOutputDevicesReadScopeLock Lock(*State);

#if PLATFORM_DESKTOP
	// Print anything that arrives after logging has shut down to at least have it in stdout.
	if (UNLIKELY(State->BufferedOutputDevices.IsEmpty() && IsEngineExitRequested()))
	{
	#if PLATFORM_WINDOWS
		_tprintf(_T("%s\n"), Data);
	#endif
		FGenericPlatformMisc::LocalPrint(Data);
		return;
	}
#endif

	const uint32 ThreadId = FPlatformTLS::GetCurrentThreadId();

	// Serialize directly to any output devices which don't require buffering
	State->BroadcastTo(ThreadId, State->UnbufferedOutputDevices, State->UnbufferedOutputDevicesCanBeUsedOnPanicThread,
		UE_PROJECTION_MEMBER(FOutputDevice, Serialize),
		Data, Verbosity, Category, RealTime);

	// Serialize to the backlog when not in panic mode. This will deadlock in panic mode when the
	// FPlatformMallocCrash allocator has been enabled and logging occurs on a non-panic thread.
	if (UNLIKELY(State->bEnableBacklog && !State->HasPanicThread()))
	{
		State->AddToBacklog(Data, Verbosity, Category, RealTime);
	}

	// Serialize to buffered output devices from the primary logging thread.
	// Lines are queued until buffered output devices are added to avoid missing early log lines.
	if (State->IsPrimaryThread(ThreadId) && !State->BufferedOutputDevices.IsEmpty())
	{
		// Verify that this is the primary thread again because another thread may have become
		// the primary thread between the previous check and the lock.
		if (FOutputDevicesPrimaryScopeLock PrimaryLock(*State); PrimaryLock.IsLocked() && State->IsPrimaryThread(ThreadId))
		{
			State->FlushBufferedItems();
			State->BroadcastTo(ThreadId, State->BufferedOutputDevices, State->BufferedOutputDevicesCanBeUsedOnPanicThread,
				UE_PROJECTION_MEMBER(FOutputDevice, Serialize),
				Data, Verbosity, Category, RealTime);
			if (UNLIKELY(State->IsPanicThread(ThreadId)))
			{
				Flush();
			}
			return;
		}
	}

	// Queue the line to serialize to buffered output devices from the primary thread.
	State->BufferedItems.Enqueue(Data, Category, Verbosity, RealTime);
	if (!State->BufferedItemCount.fetch_add(1, std::memory_order_relaxed))
	{
		State->ThreadWakeEvent.Notify();
	}
}

void FOutputDeviceRedirector::Serialize(const TCHAR* Data, ELogVerbosity::Type Verbosity, const FName& Category)
{
	Serialize(Data, Verbosity, Category, -1.0);
}

void FOutputDeviceRedirector::RedirectLog(const FName& Category, ELogVerbosity::Type Verbosity, const TCHAR* Data)
{
	Serialize(Data, Verbosity, Category, -1.0);
}

void FOutputDeviceRedirector::RedirectLog(const FLazyName& Category, ELogVerbosity::Type Verbosity, const TCHAR* Data)
{
	Serialize(Data, Verbosity, Category, -1.0);
}

UE_AUTORTFM_NOAUTORTFM
FOutputDeviceFence FOutputDeviceRedirector::CreateFence()
{
	FOutputDeviceFence Fence;
	Fence.State = State.Get();
	Fence.Index = State->CreateFence();
	return Fence;
}

UE_AUTORTFM_ALWAYS_OPEN
void FOutputDeviceRedirector::Flush()
{
	if (UE::Private::FOutputDevicesPrimaryScopeLock Lock(*State); Lock.IsLocked())
	{
		State->FlushBufferedItems();
		const uint32 ThreadId = FPlatformTLS::GetCurrentThreadId();
		State->BroadcastTo(ThreadId, State->BufferedOutputDevices, State->BufferedOutputDevicesCanBeUsedOnPanicThread, &FOutputDevice::Flush);
		State->BroadcastTo(ThreadId, State->UnbufferedOutputDevices, State->UnbufferedOutputDevicesCanBeUsedOnPanicThread, &FOutputDevice::Flush);
	}
}

UE_AUTORTFM_ALWAYS_OPEN
void FOutputDeviceRedirector::Panic()
{
	uint32 PreviousThreadId = MAX_uint32;
	const uint32 ThreadId = FPlatformTLS::GetCurrentThreadId();
	if (State->PanicThreadId.compare_exchange_strong(PreviousThreadId, ThreadId, std::memory_order_relaxed))
	{
		// Another thread may be holding the lock. Wait a while for it, but avoid waiting forever
		// because the thread holding the lock may be unable to progress. After the timeout is
		// reached, assume that it is safe enough to continue on the panic thread. There is a
		// chance that the thread holding the lock has left an output device in an unusable state
		// or will resume and crash due to a race with the panic thread. Executing on this thread
		// and having logging for most panic situations with a chance of a crash is preferable to
		// the alternative of missing logging in a panic situation.
		constexpr double WaitTime = 1.0;
		for (const double EndTime = FPlatformTime::Seconds() + WaitTime; FPlatformTime::Seconds() < EndTime;)
		{
			if (State->OutputDevicesLock.TryWriteLock())
			{
				State->OutputDevicesLock.WriteUnlock();
				break;
			}
			FPlatformProcess::Yield();
		}

		// Make the panic thread the primary thread. Neither thread can be changed after this point.
		State->PrimaryThreadId.exchange(ThreadId, std::memory_order_relaxed);

		// Flush. Every log from the panic thread after this point will also flush.
		Flush();
	}
	else if (PreviousThreadId == ThreadId)
	{
		// Calling Panic() multiple times from the panic thread is equivalent to calling Flush().
		Flush();
	}
}

UE_AUTORTFM_NOAUTORTFM // Not currently supported in a closed transaction
void FOutputDeviceRedirector::TearDown()
{
	SetCurrentThreadAsPrimaryThread();

	Flush();

	State->TryStopThread();

	TArray<FOutputDevice*> LocalBufferedDevices;
	TArray<FOutputDevice*> LocalUnbufferedDevices;

	{
		UE::Private::FOutputDevicesWriteScopeLock Lock(*State);
		LocalBufferedDevices = MoveTemp(State->BufferedOutputDevices);
		LocalUnbufferedDevices = MoveTemp(State->UnbufferedOutputDevices);
		State->BufferedOutputDevices.Empty();
		State->UnbufferedOutputDevices.Empty();
	}

	for (FOutputDevice* OutputDevice : LocalBufferedDevices)
	{
		OutputDevice->TearDown();
	}

	for (FOutputDevice* OutputDevice : LocalUnbufferedDevices)
	{
		OutputDevice->TearDown();
	}
}

UE_AUTORTFM_ALWAYS_OPEN
bool FOutputDeviceRedirector::IsBacklogEnabled() const
{
	return HasBacklogForCrash();
}

bool FOutputDeviceRedirector::HasBacklogForCrash() const
{
	FReadScopeLock Lock(State->BacklogLock);
	return State->bEnableBacklog;
}

void FOutputDeviceRedirector::SerializeBacklogForCrash(TNotNull<FOutputDevice*> OutputDevice) const
{
	State->SerializeBacklog(OutputDevice);
}

CORE_API FOutputDeviceRedirector* GetGlobalLogSingleton()
{
	return GWasGLogDestroyed ? nullptr : FOutputDeviceRedirector::Get();
}

void FOutputDeviceFence::Wait() const
{
	State->WaitFence(Index);
}

bool FOutputDeviceFence::Poll() const
{
	return Index <= State->FenceCompleteIndex.load(std::memory_order_relaxed);
}
