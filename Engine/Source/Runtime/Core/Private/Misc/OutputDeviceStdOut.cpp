// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/OutputDeviceStdOut.h"

#include "CoreGlobals.h"
#include "Logging/StructuredLog.h"
#include "Logging/StructuredLogFormat.h"
#include "Misc/AsciiSet.h"
#include "Misc/CommandLine.h"
#include "Misc/DateTime.h"
#include "Misc/OutputDeviceHelper.h"
#include "Misc/Parse.h"
#include "Serialization/CompactBinary.h"
#include "Serialization/CompactBinarySerialization.h"
#include "Serialization/CompactBinaryWriter.h"

#if PLATFORM_WINDOWS
#include "Windows/WindowsHWrapper.h"
#endif

// Several functions below are FORCENOINLINE to reduce total required stack space by limiting
// the scope of string builders and compact binary writers.

namespace UE::Logging::Private
{

#if PLATFORM_WINDOWS || PLATFORM_TCHAR_IS_UTF8CHAR || PLATFORM_TCHAR_IS_CHAR16
using StdOutCharType = UTF8CHAR;
#else
using StdOutCharType = WIDECHAR;
#endif

static void WriteLineToStdOut(FUtf8StringBuilderBase& Line)
{
	fwrite(Line.GetData(), 1, Line.Len(), stdout);
	fflush(stdout);
}

#if PLATFORM_WINDOWS || !(PLATFORM_TCHAR_IS_UTF8CHAR || PLATFORM_TCHAR_IS_CHAR16)
static void WriteLineToStdOut(FWideStringBuilderBase& Line)
{
#if PLATFORM_USE_LS_SPEC_FOR_WIDECHAR
	// printf prints wchar_t strings just fine with %ls, while mixing printf()/wprintf() is not recommended (see https://stackoverflow.com/questions/8681623/printf-and-wprintf-in-single-c-code)
	printf("%ls", *Line);
#else
	wprintf(TEXT("%s"), *Line);
#endif
	fflush(stdout);
}
#endif

#if PLATFORM_WINDOWS
static void WriteLineToConsole(FWideStringBuilderBase& Line)
{
	WriteConsoleW(GetStdHandle(STD_OUTPUT_HANDLE), *Line, Line.Len(), nullptr, nullptr);
}
#endif

FORCEINLINE static void AddFormatPrefix(FUtf8StringBuilderBase& Format, bool bShowCategory, bool bShowVerbosity)
{
	if (bShowCategory)
	{
		Format << ANSITEXTVIEW("{_channel}: ");
	}
	if (bShowVerbosity)
	{
		Format << ANSITEXTVIEW("{_severity}: ");
	}
}

FORCEINLINE static void AddMessagePrefix(FUtf8StringBuilderBase& Message, const FName& Category, ELogVerbosity::Type Verbosity, bool bShowCategory, bool bShowVerbosity)
{
	if (bShowCategory)
	{
		Message << Category << ANSITEXTVIEW(": ");
	}
	if (bShowVerbosity)
	{
		Message << ToString(Verbosity) << ANSITEXTVIEW(": ");
	}
}

FORCEINLINE static void AddMessageFields(FCbWriter& Writer, const FName& Category, ELogVerbosity::Type Verbosity, bool bShowCategory, bool bShowVerbosity)
{
	if (bShowCategory)
	{
		Writer.BeginObject(ANSITEXTVIEW("_channel"));
		Writer.AddString(ANSITEXTVIEW("$type"), ANSITEXTVIEW("Channel"));
		Writer.AddString(ANSITEXTVIEW("$text"), WriteToUtf8String<64>(Category));
		Writer.EndObject();
	}
	if (bShowVerbosity)
	{
		Writer.BeginObject(ANSITEXTVIEW("_severity"));
		Writer.AddString(ANSITEXTVIEW("$type"), ANSITEXTVIEW("Severity"));
		Writer.AddString(ANSITEXTVIEW("$text"), ToString(Verbosity));
		Writer.EndObject();
	}
}

FORCENOINLINE static void AddMessage(FCbWriter& Writer, const TCHAR* V, ELogVerbosity::Type Verbosity, const FName& Category, bool bShowCategory, bool bShowVerbosity)
{
	TUtf8StringBuilder<512> Message;
	AddMessagePrefix(Message, Category, Verbosity, bShowCategory, bShowVerbosity);
	Message << V;
	Writer.AddString(ANSITEXTVIEW("message"), Message);
}

FORCENOINLINE static void AddFormat(FCbWriter& Writer, const TCHAR* Message, bool bShowCategory, bool bShowVerbosity)
{
	TUtf8StringBuilder<512> Format;
	AddFormatPrefix(Format, bShowCategory, bShowVerbosity);

	// Escape {} in Message
	for (constexpr FAsciiSet Brackets("{}");;)
	{
		const TCHAR* End = FAsciiSet::FindFirstOrEnd(Message, Brackets);
		Format.Append(Message, UE_PTRDIFF_TO_INT32(End - Message));
		if (!*End)
		{
			break;
		}
		Format.AppendChar(*End);
		Format.AppendChar(*End);
		Message = End + 1;
	}

	Writer.AddString(ANSITEXTVIEW("format"), Format);
}

FORCENOINLINE static void AddMessage(FCbWriter& Writer, const UE::FLogRecord& Record, bool bShowCategory, bool bShowVerbosity)
{
	TUtf8StringBuilder<512> Message;
	AddMessagePrefix(Message, Record.GetCategory(), Record.GetVerbosity(), bShowCategory, bShowVerbosity);
	Record.FormatMessageTo(Message);
	Writer.AddString(ANSITEXTVIEW("message"), Message);
}

static FAnsiStringView GetLevel(ELogVerbosity::Type Verbosity)
{
	switch (Verbosity & ELogVerbosity::VerbosityMask)
	{
	case ELogVerbosity::Fatal:
		return ANSITEXTVIEW("Critical");
	case ELogVerbosity::Error:
		return ANSITEXTVIEW("Error");
	case ELogVerbosity::Warning:
		return ANSITEXTVIEW("Warning");
	case ELogVerbosity::Display:
	case ELogVerbosity::Log:
	default:
		return ANSITEXTVIEW("Information");
	case ELogVerbosity::Verbose:
	case ELogVerbosity::VeryVerbose:
		return ANSITEXTVIEW("Debug");
	}
}

} // UE::Logging::Private

#if PLATFORM_WINDOWS
static bool IsStdOutAttachedToConsole()
{
	if (HANDLE Handle = GetStdHandle(STD_OUTPUT_HANDLE); Handle != INVALID_HANDLE_VALUE)
	{
		if (DWORD FileType = GetFileType(Handle); FileType == FILE_TYPE_CHAR)
		{
			return true;
		}
	}
	return false;
}
#endif

FOutputDeviceStdOutput::FOutputDeviceStdOutput()
{
#if PLATFORM_WINDOWS
	bIsConsoleOutput = IsStdOutAttachedToConsole() && !FParse::Param(FCommandLine::Get(), TEXT("GenericConsoleOutput"));
#endif

	bIsJsonOutput = FParse::Param(FCommandLine::Get(), TEXT("JsonStdOut"));
	if (!bIsJsonOutput)
	{
		if (FString Env = FPlatformMisc::GetEnvironmentVariable(TEXT("UE_LOG_JSON_TO_STDOUT")); !Env.IsEmpty())
		{
			bIsJsonOutput = FCString::Atoi(*Env) != 0;
		}
	}

	if (FParse::Param(FCommandLine::Get(), TEXT("AllowStdOutLogVerbosity")))
	{
		AllowedLogVerbosity = ELogVerbosity::Log;
	}

	if (FParse::Param(FCommandLine::Get(), TEXT("FullStdOutLogOutput")))
	{
		AllowedLogVerbosity = ELogVerbosity::All;
	}

	if (stdout == nullptr)
	{
		PRAGMA_DISABLE_UNREACHABLE_CODE_WARNINGS
		AllowedLogVerbosity = ELogVerbosity::NoLogging;
		PRAGMA_RESTORE_UNREACHABLE_CODE_WARNINGS
	}
}

void FOutputDeviceStdOutput::Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const FName& Category, double Time)
{
	if (Verbosity <= AllowedLogVerbosity)
	{
		return bIsJsonOutput ? SerializeAsJson(V, Verbosity, Category, Time) : SerializeAsText(V, Verbosity, Category, Time);
	}
}

void FOutputDeviceStdOutput::SerializeRecord(const UE::FLogRecord& Record)
{
	if (Record.GetVerbosity() <= AllowedLogVerbosity)
	{
		return bIsJsonOutput ? SerializeRecordAsJson(Record) : SerializeRecordAsText(Record);
	}
}

FORCENOINLINE void FOutputDeviceStdOutput::SerializeAsText(const TCHAR* V, ELogVerbosity::Type Verbosity, const FName& Category, double Time)
{
	using namespace UE::Logging::Private;

	auto FormatLine = [V, Verbosity, &Category, Time]<typename CharType>(TStringBuilderBase<CharType>& Line) -> TStringBuilderBase<CharType>&
	{
		FOutputDeviceHelper::AppendFormatLogLine(Line, Verbosity, Category, V, GPrintLogTimes, Time);
		Line.AppendChar('\n');
		return Line;
	};

#if PLATFORM_WINDOWS
	if (bIsConsoleOutput)
	{
		TStringBuilderWithBuffer<WIDECHAR, 512> Line;
		WriteLineToConsole(FormatLine(Line));
		return;
	}
#endif

	TStringBuilderWithBuffer<StdOutCharType, 512> Line;
	WriteLineToStdOut(FormatLine(Line));
}

FORCENOINLINE void FOutputDeviceStdOutput::SerializeRecordAsText(const UE::FLogRecord& Record)
{
	TStringBuilder<512> V;
	Record.FormatMessageTo(V);
	Serialize(*V, Record.GetVerbosity(), Record.GetCategory(), -1.0);
}

FORCENOINLINE void FOutputDeviceStdOutput::SerializeAsJson(const TCHAR* V, ELogVerbosity::Type Verbosity, const FName& Category, double Time)
{
	using namespace UE::Logging::Private;

	const bool bShowCategory = GPrintLogCategory && !Category.IsNone();
	const bool bShowVerbosity = GPrintLogVerbosity && (Verbosity & ELogVerbosity::VerbosityMask) != ELogVerbosity::Log;

	TCbWriter<1024> Writer;
	Writer.BeginObject();
	Writer.AddDateTime(ANSITEXTVIEW("time"), FDateTime::UtcNow());
	if (const uint64 Frame = GFrameCounter)
	{
		Writer.AddInteger(ANSITEXTVIEW("frame"), Frame);
	}
	Writer.AddString(ANSITEXTVIEW("level"), GetLevel(Verbosity));
	AddMessage(Writer, V, Verbosity, Category, bShowCategory, bShowVerbosity);
	if (bShowCategory || bShowVerbosity)
	{
		Writer.BeginObject(ANSITEXTVIEW("properties"));
		AddMessageFields(Writer, Category, Verbosity, bShowCategory, bShowVerbosity);
		Writer.EndObject();

		AddFormat(Writer, V, bShowCategory, bShowVerbosity);
	}
	Writer.EndObject();

	WriteAsJson(Writer);
}

FORCENOINLINE void FOutputDeviceStdOutput::SerializeRecordAsJson(const UE::FLogRecord& Record)
{
	using namespace UE;
	using namespace UE::Logging::Private;

	const bool bShowCategory = GPrintLogCategory && !Record.GetCategory().IsNone();
	const bool bShowVerbosity = GPrintLogVerbosity && (Record.GetVerbosity() & ELogVerbosity::VerbosityMask) != ELogVerbosity::Log;

	TCbWriter<1024> Writer;
	Writer.BeginObject();
	Writer.AddDateTime(ANSITEXTVIEW("time"), Record.GetTime().GetUtcTime());
	if (const uint64 Frame = GFrameCounter)
	{
		Writer.AddInteger(ANSITEXTVIEW("frame"), Frame);
	}
	Writer.AddString(ANSITEXTVIEW("level"), GetLevel(Record.GetVerbosity()));
	AddMessage(Writer, Record, bShowCategory, bShowVerbosity);
	if (bShowCategory || bShowVerbosity || Record.GetFields())
	{
		TUtf8StringBuilder<512> Format;
		AddFormatPrefix(Format, bShowCategory, bShowVerbosity);

		Writer.BeginObject(ANSITEXTVIEW("properties"));
		AddMessageFields(Writer, Record.GetCategory(), Record.GetVerbosity(), bShowCategory, bShowVerbosity);
		if (const TCHAR* TextNamespace = Record.GetTextNamespace())
		{
			Writer.AddString(ANSITEXTVIEW("_ns"), TextNamespace);
		}
		if (const TCHAR* TextKey = Record.GetTextKey())
		{
			Writer.AddString(ANSITEXTVIEW("_key"), TextKey);
		}
		Record.ConvertToCommonLog(Format, Writer);
		Writer.EndObject();

		Writer.AddString(ANSITEXTVIEW("format"), Format);
	}
	Writer.EndObject();

	WriteAsJson(Writer);
}

FORCENOINLINE void FOutputDeviceStdOutput::WriteAsJson(const FCbWriter& Writer)
{
	using namespace UE::Logging::Private;

	TArray<uint8, TInlineAllocator64<512>> Buffer;
	Buffer.AddUninitialized((int64)Writer.GetSaveSize());
	FCbFieldView Object = Writer.Save(MakeMemoryView(Buffer));

	auto FormatLine = []<typename CharType>(const FCbFieldView& Field, TStringBuilderBase<CharType>& Line) -> TStringBuilderBase<CharType>&
	{
		CompactBinaryToCompactJson(Field, Line);
		Line.AppendChar('\n');
		return Line;
	};

#if PLATFORM_WINDOWS
	if (bIsConsoleOutput)
	{
		TStringBuilderWithBuffer<WIDECHAR, 512> Line;
		WriteLineToConsole(FormatLine(Object, Line));
		return;
	}
#endif

	TStringBuilderWithBuffer<StdOutCharType, 512> Line;
	WriteLineToStdOut(FormatLine(Object, Line));
}
