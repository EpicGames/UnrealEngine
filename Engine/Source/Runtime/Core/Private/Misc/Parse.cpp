// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/Parse.h"
#include "Misc/DateTime.h"
#include "HAL/PlatformProcess.h"
#include "UObject/NameTypes.h"
#include "HAL/FileManager.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Containers/Set.h"
#include "Internationalization/Text.h"
#include "Misc/AsciiSet.h"
#include "Misc/CoreDelegates.h"
#include "Misc/Guid.h"
#include "Misc/OutputDeviceNull.h"
#include "Misc/StringBuilder.h"
#include "HAL/IConsoleManager.h"
#include "Containers/LazyPrintf.h"
#include "Containers/StringView.h"

#if !UE_BUILD_SHIPPING 
/**
 * Needed for the console command "DumpConsoleCommands"
 * How it works:
 *   - GConsoleCommandLibrary is set to point at a local instance of ConsoleCommandLibrary
 *   - a dummy command search is triggered which gathers all commands in a hashed set
 *   - sort all gathered commands in human friendly way
 *   - log all commands
 *   - GConsoleCommandLibrary is set 0
 */
class ConsoleCommandLibrary
{
public:
	ConsoleCommandLibrary(FStringView InPrefix);

	~ConsoleCommandLibrary();

	void OnParseCommand(const TCHAR* Cmd)
	{
		if (FCString::Strnicmp(Cmd, Prefix.GetData(), Prefix.Len()) == 0)
		{
			KnownNames.Add(Cmd);
		}
	}

	FStringView   Prefix;
	TSet<FString> KnownNames;
};

// 0 if gathering of names is deactivated
ConsoleCommandLibrary* GConsoleCommandLibrary;

ConsoleCommandLibrary::ConsoleCommandLibrary(FStringView InPrefix) : Prefix(InPrefix)
{
	// activate name gathering
	GConsoleCommandLibrary = this;
}

ConsoleCommandLibrary::~ConsoleCommandLibrary()
{
	// deactivate name gathering
	GConsoleCommandLibrary = 0;
}

bool ConsoleCommandLibrary_DumpLibrary(UWorld* InWorld, FExec& SubSystem, const FString& Pattern, FOutputDevice& Ar)
{
	TStringBuilder<32> Prefix;
	Prefix << FStringView(Pattern).LeftChop(1);
	return ConsoleCommandLibrary_DumpLibrary(InWorld, SubSystem, *Prefix, Ar);
}

bool ConsoleCommandLibrary_DumpLibrary(UWorld* InWorld, FExec& SubSystem, const TCHAR* Prefix, FOutputDevice& Ar)
{
	// Install a global handler to scrape unregistered commands as FExec implementations call FParse::Command
	ConsoleCommandLibrary LocalConsoleCommandLibrary(Prefix);

	// Gather unregistered commands
	bool bExecuted = false;
	{
 		TStringBuilder<32> FakeCmd;
		FakeCmd << Prefix << '*';

		FOutputDeviceNull Null;
		bExecuted = SubSystem.Exec(InWorld, *FakeCmd, Null);
	}

	auto VisitConsoleCommand = [&Sink = LocalConsoleCommandLibrary.KnownNames](const TCHAR *Name, IConsoleObject* Object)
	{
		if (!Object->TestFlags(ECVF_Unregistered) && Object->AsCommand())
		{
			Sink.Add(Name);
		}
	};

	// Gather registered commands
	IConsoleManager::Get().ForEachConsoleObjectThatStartsWith(FConsoleObjectVisitor::CreateLambda(VisitConsoleCommand), Prefix);

	LocalConsoleCommandLibrary.KnownNames.Sort( TLess<FString>() );

	for(TSet<FString>::TConstIterator It(LocalConsoleCommandLibrary.KnownNames); It; ++It)
	{
		const FString Name = *It;

		Ar.Logf(TEXT("%s"), *Name);
	}
	Ar.Logf(TEXT(""));

	// the fake command (e.g. Motion*) should not really trigger the execution
	if(bExecuted)
	{
		Ar.Logf(TEXT("ERROR: The function was supposed to only find matching commands but not have any side effect."));
		Ar.Logf(TEXT("However Exec() returned true which means we either executed a command or the command parsing returned true where it shouldn't."));
	}

	return true;
}

bool ConsoleCommandLibrary_DumpLibraryHTML(UWorld* InWorld, FExec& SubSystem, const FString& OutPath)
{
	FStringView Prefix = TEXT("");

	// Install a global handler to scrape unregistered commands as FExec implementations call FParse::Command
	ConsoleCommandLibrary LocalConsoleCommandLibrary(Prefix);

	// Gather unregistered commands
	bool bExecuted = false;
	{
		const TCHAR* FakeCmd = TEXT("*");

		FOutputDeviceNull Null;
		bExecuted = SubSystem.Exec(InWorld, FakeCmd, Null);
	}

	auto VisitConsoleObject = [&Sink = LocalConsoleCommandLibrary.KnownNames](const TCHAR *Name, IConsoleObject* Object)
	{
		if (!Object->TestFlags(ECVF_Unregistered))
		{
			Sink.Add(Name);
		}
	};

	// Gather registered variables and commands
	IConsoleManager::Get().ForEachConsoleObjectThatStartsWith(FConsoleObjectVisitor::CreateLambda(VisitConsoleObject));

	LocalConsoleCommandLibrary.KnownNames.Sort( TLess<FString>() );

	FString TemplateFilename = FPaths::Combine(FPlatformProcess::BaseDir(), TEXT("../../Documentation/Extras"), TEXT("ConsoleHelpTemplate.html"));
	FString TemplateFile;
	if(FFileHelper::LoadFileToString(TemplateFile, *TemplateFilename, FFileHelper::EHashOptions::EnableVerify | FFileHelper::EHashOptions::ErrorMissingHash) )
	{
		// todo: do we need to create the directory?
#if ALLOW_DEBUG_FILES
		FArchive* File = IFileManager::Get().CreateDebugFileWriter(*OutPath);
#else
		FArchive* File = nullptr;
#endif

		if (File)
		{
			FLazyPrintf LazyPrintf(*TemplateFile);

			// title
			LazyPrintf.PushParam(TEXT("UE5 Console Variables and Commands"));
			// headline
			LazyPrintf.PushParam(TEXT("Unreal Engine 5 Console Variables and Commands"));
			// generated by
			LazyPrintf.PushParam(TEXT("Unreal Engine 5 console command 'Help'"));
			// version
			LazyPrintf.PushParam(TEXT("0.95"));
			// date
			LazyPrintf.PushParam(*FDateTime::Now().ToString());

			FString AllData;

			for (TSet<FString>::TConstIterator It(LocalConsoleCommandLibrary.KnownNames); It; ++It)
			{
				const FString& Name = *It;

				auto Element = IConsoleManager::Get().FindConsoleObject(*Name);

				if (Element)
				{
					// console command or variable

					FString Help = Element->GetHelp();

					Help = Help.ReplaceCharWithEscapedChar();

					const TCHAR* ElementType = TEXT("Unknown");

					if(Element->AsVariable())
					{
						ElementType = TEXT("Var"); 
					}
					else if(Element->AsCommand())
					{
						ElementType = TEXT("Cmd"); 
					}

					//{name: "r.SetRes", help:"To change the screen/window resolution."},
					FString DataLine = FString::Printf(TEXT("{name: \"%s\", help:\"%s\", type:\"%s\"},\r\n"), *Name, *Help, ElementType);

					AllData += DataLine;
				}
				else
				{
					// Exec command (better we change them to use the new method as it has better help and is more convenient to use)
					//{name: "", help:"To change the screen/window resolution."},
					FString DataLine = FString::Printf(TEXT("{name: \"%s\", help:\"Sorry: Exec commands have no help\", type:\"Exec\"},\r\n"), *Name);

					AllData += DataLine;
				}
			}

			LazyPrintf.PushParam(*AllData);

			FTCHARToUTF8 UTF8Help(*LazyPrintf.GetResultString());
			File->Serialize((ANSICHAR*)UTF8Help.Get(), UTF8Help.Length());

			delete File;
			File = 0;

			return true;
		}
	}

	return false;
/*
	// the pattern (e.g. Motion*) should not really trigger the execution
	if(bExecuted)
	{
		Ar.Logf(TEXT("ERROR: The function was supposed to only find matching commands but not have any side effect."));
		Ar.Logf(TEXT("However Exec() returned true which means we either executed a command or the command parsing returned true where it shouldn't."));
	}
*/
}
#endif // UE_BUILD_SHIPPING

//
// Get a string from a text string.
//
bool FParse::Value(
	const TCHAR*	Stream,
	const TCHAR*	Match,
	TCHAR*			Value,
	int32			MaxLen,
	bool			bShouldStopOnSeparator,
	const TCHAR**	OptStreamGotTo
)
{
	if (MaxLen == 0)
	{
		return false;
	}
	check(Value && MaxLen > 0);

	bool bSuccess = false;
	int32 MatchLen = FCString::Strlen(Match);

	if (OptStreamGotTo)
	{
		*OptStreamGotTo = nullptr;
	}

	const TCHAR* FoundInStream = FCString::Strifind(Stream, Match, true);
	if (FoundInStream == nullptr)
	{
		Value[0] = TCHAR('\0');
		return false;
	}

	const TCHAR* ValueStartInStream = FoundInStream + MatchLen;
	const TCHAR* ValueEndInStream;

	// Check for quoted arguments' string with spaces
	// -Option="Value1 Value2"
	//         ^~~~Start
	const bool bArgumentsQuoted = *ValueStartInStream == '"';

	if (bArgumentsQuoted)
	{
		// Skip quote character if only params were quoted.
		ValueStartInStream += 1;
		ValueEndInStream = FCString::Strstr(ValueStartInStream, TEXT("\x22"));

		if (ValueEndInStream == nullptr)
		{
			// this should probably log a warning if bArgumentsQuoted is true, as we started with a '"' and didn't find the terminating one.
			ValueEndInStream = FoundInStream + FCString::Strlen(FoundInStream);
		}
	}
	else
	{
		// Skip initial whitespace
		const TCHAR* WhiteSpaceChars = TEXT(" \r\n\t");
		ValueStartInStream += FCString::Strspn(ValueStartInStream, WhiteSpaceChars);

		// Non-quoted string without spaces.
		const TCHAR* TerminatingChars = bShouldStopOnSeparator ? TEXT(",) \r\n\t") : WhiteSpaceChars;
		ValueEndInStream = ValueStartInStream + FCString::Strcspn(ValueStartInStream, TerminatingChars);
	}

	int32 ValueLength = FMath::Min<int32>(MaxLen - 1, UE_PTRDIFF_TO_INT32(ValueEndInStream - ValueStartInStream));
	// It is possible for ValueLength to be 0.
	// FCString::Strncpy asserts that its copying at least 1 char, memcpy has no such constraint.
	FMemory::Memcpy(Value, ValueStartInStream, sizeof(Value[0]) * ValueLength);
	Value[ValueLength] = TCHAR('\0');

	if (OptStreamGotTo)
	{
		if (bArgumentsQuoted && *ValueEndInStream == '"')
		{
			++ValueEndInStream;
		}

		*OptStreamGotTo = ValueEndInStream;
	}

	return true;
}

//
// Checks if a command-line parameter exists in the stream.
//
bool FParse::Param( const TCHAR* Stream, const TCHAR* Param )
{
	if (*Param == '-' || *Param == '/')
	{
		++Param;
	}

	const TCHAR* Start = Stream;
	if( *Stream )
	{
		while( (Start=FCString::Strifind(Start,Param,true)) != NULL )
		{
			if( Start>Stream && (Start[-1]=='-' || Start[-1]=='/') && 
				(Stream > (Start - 2) || FChar::IsWhitespace(Start[-2]))) // Reject if the character before '-' or '/' is not a whitespace
			{
				const TCHAR* End = Start + FCString::Strlen(Param);
				if ( End == NULL || *End == 0 || FChar::IsWhitespace(*End) )
				{
					return true;
				}
			}

			Start++;
		}
	}
	return false;
}

// 
// Parse a string.
//
bool FParse::Value(const TCHAR* Stream, const TCHAR* Match, FString& Value, bool bShouldStopOnSeparator, const TCHAR** OptStreamGotTo)
{
	if (!Stream)
	{
		return false;
	}

	int32 StreamLen = FCString::Strlen(Stream);
	if (StreamLen > 0)
	{
		TArray<TCHAR, TInlineAllocator<4096>> ValueCharArray;
		ValueCharArray.AddUninitialized(StreamLen + 1);
		ValueCharArray[0] = TCHAR('\0');

		if( FParse::Value(Stream, Match, ValueCharArray.GetData(), ValueCharArray.Num(), bShouldStopOnSeparator, OptStreamGotTo) )
		{
			Value = FString(ValueCharArray.GetData());
			return true;
		}
	}

	return false;
}

template<class T>
bool ParseQuotedString( const TCHAR* Buffer, T& Value, int32* OutNumCharsRead)
{
	if (OutNumCharsRead)
	{
		*OutNumCharsRead = 0;
	}

	const TCHAR* Start = Buffer;

	// Require opening quote
	if (*Buffer++ != TCHAR('"'))
	{
		return false;
	}

	constexpr FAsciiSet StopCharacters = FAsciiSet("\"\n\r") + '\0';
	constexpr FAsciiSet StopAndEscapeCharacters = StopCharacters + '\\';
	auto ShouldParse = [=](const TCHAR Ch) { return StopCharacters.Test(Ch) == 0; };
	
	while (true)
	{
		// Append unescaped substring
		const TCHAR* UnescapedEnd = FAsciiSet::FindFirstOrEnd(Buffer, StopAndEscapeCharacters);
		FStringView UnescapedSubstring(Buffer, int32(UnescapedEnd - Buffer));
		Value += UnescapedSubstring;
		Buffer = UnescapedEnd;

		if (*Buffer != '\\') // Found a stop character
		{
			break;
		}
		else if (*++Buffer == TCHAR('\\')) // escaped backslash "\\"
		{
			Value += TCHAR('\\');
			++Buffer;
		}
		else if (*Buffer == TCHAR('"')) // escaped double quote "\""
		{
			Value += TCHAR('"');
			++Buffer;
		}
		else if (*Buffer == TCHAR('\'')) // escaped single quote "\'"
		{
			Value += TCHAR('\'');
			++Buffer;
		}
		else if (*Buffer == TCHAR('n')) // escaped newline
		{
			Value += TCHAR('\n');
			++Buffer;
		}
		else if (*Buffer == TCHAR('r')) // escaped carriage return
		{
			Value += TCHAR('\r');
			++Buffer;
		}
		else if (*Buffer == TCHAR('t')) // escaped tab
		{
			Value += TCHAR('\t');
			++Buffer;
		}
		else if (FChar::IsOctDigit(*Buffer)) // octal sequence (\012)
		{
			TStringBuilder<16> OctSequence;
			while (ShouldParse(*Buffer) && FChar::IsOctDigit(*Buffer) && OctSequence.Len() < 3) // Octal sequences can only be up-to 3 digits long
			{
				OctSequence += *Buffer++;
			}

			Value += (TCHAR)FCString::Strtoi(*OctSequence, nullptr, 8);
		}
		else if (*Buffer == TCHAR('x') && FChar::IsHexDigit(*(Buffer + 1))) // hex sequence (\xBEEF)
		{
			++Buffer;

			TStringBuilder<16> HexSequence;
			while (ShouldParse(*Buffer) && FChar::IsHexDigit(*Buffer))
			{
				HexSequence += *Buffer++;
			}

			Value += (TCHAR)FCString::Strtoi(*HexSequence, nullptr, 16);
		}
		else if (*Buffer == TCHAR('u') && FChar::IsHexDigit(*(Buffer + 1))) // UTF-16 sequence (\u1234)
		{
			++Buffer;

			TStringBuilder<4> UnicodeSequence;
			while (ShouldParse(*Buffer) && FChar::IsHexDigit(*Buffer) && UnicodeSequence.Len() < 4) // UTF-16 sequences can only be up-to 4 digits long
			{
				UnicodeSequence += *Buffer++;
			}

			const UTF16CHAR Utf16Char = static_cast<UTF16CHAR>(FCString::Strtoi(*UnicodeSequence, nullptr, 16));
			const FUTF16ToTCHAR Utf16Str(&Utf16Char, /* Len */ 1);
			Value += FStringView(Utf16Str.Get(), Utf16Str.Length());
		}
		else if (*Buffer == TCHAR('U') && FChar::IsHexDigit(*(Buffer + 1))) // UTF-32 sequence (\U12345678)
		{
			++Buffer;

			TStringBuilder<8> UnicodeSequence;
			while (ShouldParse(*Buffer) && FChar::IsHexDigit(*Buffer) && UnicodeSequence.Len() < 8) // UTF-32 sequences can only be up-to 8 digits long
			{
				UnicodeSequence += *Buffer++;
			}

			const UTF32CHAR Utf32Char = static_cast<UTF32CHAR>(FCString::Strtoi(*UnicodeSequence, nullptr, 16));
			const FUTF32ToTCHAR Utf32Str(&Utf32Char, /* Len */ 1);
			Value += FStringView(Utf32Str.Get(), Utf32Str.Length());
		}
		else // unhandled escape sequence
		{
			Value += TCHAR('\\');
			Value += *Buffer++;
		}
	}

	// Require closing quote
	if (*Buffer++ != TCHAR('"'))
	{
		return false;
	}

	if (OutNumCharsRead)
	{
		*OutNumCharsRead = UE_PTRDIFF_TO_INT32(Buffer - Start);
	}

	return true;
}

bool FParse::QuotedString( const TCHAR* Buffer, FString& Value, int32* OutNumCharsRead )
{
	return ParseQuotedString(Buffer, Value, OutNumCharsRead);
}

bool FParse::QuotedString( const TCHAR* Buffer, FStringBuilderBase& Value, int32* OutNumCharsRead )
{
	return ParseQuotedString(Buffer, Value, OutNumCharsRead);
}

// 
// Parse an Text token
// This is expected to in the form NSLOCTEXT("Namespace","Key","SourceString") or LOCTEXT("Key","SourceString")
//
bool FParse::Text( const TCHAR* Buffer, FText& Value, const TCHAR* Namespace )
{
	return FTextStringHelper::ReadFromBuffer(Buffer, Value, Namespace) != nullptr;
}

// 
// Parse an Text.
// This is expected to in the form NSLOCTEXT("Namespace","Key","SourceString") or LOCTEXT("Key","SourceString")
//
bool FParse::Value(const TCHAR* Stream, const TCHAR* Match, FText& Value, const TCHAR* Namespace)
{
	// The FText 
	Stream = FCString::Strifind( Stream, Match );
	if( Stream )
	{
		Stream += FCString::Strlen( Match );
		return FParse::Text( Stream, Value, Namespace );
	}

	return false;
}

//
// Parse a quadword.
//
bool FParse::Value(const TCHAR* Stream, const TCHAR* Match, uint64& Value, const TCHAR** OptStreamGotTo)
{
	return FParse::Value( Stream, Match, *(int64*)&Value, OptStreamGotTo);
}

//
// Parse a signed quadword.
//
bool FParse::Value(const TCHAR* Stream, const TCHAR* Match, int64& Value, const TCHAR** OptStreamGotTo)
{
	TCHAR Temp[4096] = {};
	TCHAR* Ptr = Temp;
	if (FParse::Value( Stream, Match, Temp, UE_ARRAY_COUNT(Temp), true, OptStreamGotTo))
	{
		Value = 0;
		bool Negative = (*Ptr=='-');
		Ptr += Negative;
		while( *Ptr>='0' && *Ptr<='9' )
			Value = Value*10 + *Ptr++ - '0';
		if( Negative )
			Value = -Value;
		return true;
	}
	else
	{
		return false;
	}
}

//
// Get a name.
//
bool FParse::Value(const TCHAR* Stream, const TCHAR* Match, FName& Name, const TCHAR** OptStreamGotTo)
{
	TCHAR TempStr[NAME_SIZE];

	if (!FParse::Value(Stream, Match, TempStr, NAME_SIZE, true, OptStreamGotTo))
	{
		return false;
	}

	Name = FName(TempStr);

	return true;
}

//
// Get a uint32.
//
bool FParse::Value(const TCHAR* Stream, const TCHAR* Match, uint32& Value, const TCHAR** OptStreamGotTo)
{
	TCHAR Temp[256];
	if (!FParse::Value(Stream, Match, Temp, UE_ARRAY_COUNT(Temp), true, OptStreamGotTo))
	{
		return false;
	}
	TCHAR* End_NotUsed;

	Value = FCString::Strtoi(Temp, &End_NotUsed, 10 );

	return true;
}

//
// Get a byte.
//
bool FParse::Value(const TCHAR* Stream, const TCHAR* Match, uint8& Value, const TCHAR** OptStreamGotTo)
{
	TCHAR Temp[256];
	if (!FParse::Value(Stream, Match, Temp, UE_ARRAY_COUNT(Temp), true, OptStreamGotTo))
	{
		return false;
	}

	Value = (uint8)FCString::Atoi( Temp );
	return Value!=0 || FChar::IsDigit(Temp[0]);
}

//
// Get a signed byte.
//
bool FParse::Value(const TCHAR* Stream, const TCHAR* Match, int8& Value, const TCHAR** OptStreamGotTo)
{
	TCHAR Temp[256];
	if (!FParse::Value(Stream, Match, Temp, UE_ARRAY_COUNT(Temp), true, OptStreamGotTo))
	{
		return false;
	}

	Value = (int8)FCString::Atoi( Temp );
	return Value!=0 || FChar::IsDigit(Temp[0]);
}

//
// Get a word.
//
bool FParse::Value(const TCHAR* Stream, const TCHAR* Match, uint16& Value, const TCHAR** OptStreamGotTo)
{
	TCHAR Temp[256];
	if (!FParse::Value(Stream, Match, Temp, UE_ARRAY_COUNT(Temp), true, OptStreamGotTo))
	{
		return false;
	}

	Value = (uint16)FCString::Atoi( Temp );
	return Value!=0 || FChar::IsDigit(Temp[0]);
}

//
// Get a signed word.
//
bool FParse::Value(const TCHAR* Stream, const TCHAR* Match, int16& Value, const TCHAR** OptStreamGotTo)
{
	TCHAR Temp[256];
	if (!FParse::Value(Stream, Match, Temp, UE_ARRAY_COUNT(Temp), true, OptStreamGotTo))
	{
		return false;
	}

	Value = (int16)FCString::Atoi( Temp );
	return Value!=0 || FChar::IsDigit(Temp[0]);
}

//
// Get a floating-point number.
//
bool FParse::Value(const TCHAR* Stream, const TCHAR* Match, float& Value, const TCHAR** OptStreamGotTo)
{
	TCHAR Temp[256];
	if (!FParse::Value(Stream, Match, Temp, UE_ARRAY_COUNT(Temp), true, OptStreamGotTo))
	{
		return false;
	}

	Value = FCString::Atof( Temp );
	return true;
}

//
// Get a double precision floating-point number.
//
bool FParse::Value(const TCHAR* Stream, const TCHAR* Match, double& Value, const TCHAR** OptStreamGotTo)
{
	TCHAR Temp[256];
	if (!FParse::Value(Stream, Match, Temp, UE_ARRAY_COUNT(Temp), true, OptStreamGotTo))
	{
		return false;
	}

	Value = FCString::Atod(Temp);
	return true;
}


//
// Get a signed double word.
//
bool FParse::Value(const TCHAR* Stream, const TCHAR* Match, int32& Value, const TCHAR** OptStreamGotTo)
{
	TCHAR Temp[256];
	if (!FParse::Value(Stream, Match, Temp, UE_ARRAY_COUNT(Temp), true, OptStreamGotTo))
	{
		return false;
	}

	Value = FCString::Atoi( Temp );
	return true;
}

//
// Get a boolean value.
//
bool FParse::Bool(const TCHAR* Stream, const TCHAR* Match, bool& OnOff, const TCHAR** OptStreamGotTo)
{
	TCHAR TempStr[16];
	if (FParse::Value(Stream, Match, TempStr, UE_ARRAY_COUNT(TempStr), true, OptStreamGotTo))
	{
		OnOff = FCString::ToBool(TempStr);
		return true;
	}
	else
	{
		return false;
	}
}

//
// Get a globally unique identifier.
//
bool FParse::Value(const TCHAR* Stream, const TCHAR* Match, struct FGuid& Guid, const TCHAR** OptStreamGotTo)
{
	TCHAR Temp[256];
	if (!FParse::Value(Stream, Match, Temp, UE_ARRAY_COUNT(Temp)))
	{
		return false;
	}

	Guid.A = Guid.B = Guid.C = Guid.D = 0;
	if( FCString::Strlen(Temp)==32 )
	{
		TCHAR* End;
		Guid.D = FCString::Strtoi( Temp+24, &End, 16 ); Temp[24] = TCHAR('\0');
		Guid.C = FCString::Strtoi( Temp+16, &End, 16 ); Temp[16] = TCHAR('\0');
		Guid.B = FCString::Strtoi( Temp+8,  &End, 16 ); Temp[8 ] = TCHAR('\0');
		Guid.A = FCString::Strtoi( Temp+0,  &End, 16 ); Temp[0 ] = TCHAR('\0');
	}
	return true;
}


//
// Sees if Stream starts with the named command.  If it does,
// skips through the command and blanks past it.  Returns 1 of match,
// 0 if not.
//
bool FParse::Command( const TCHAR** Stream, const TCHAR* Match, bool bParseMightTriggerExecution )
{
#if !UE_BUILD_SHIPPING
	if(GConsoleCommandLibrary)
	{
		GConsoleCommandLibrary->OnParseCommand(Match);
		
		if(bParseMightTriggerExecution)
		{
			// Better we fail the test - we only wanted to find all commands.
			return false;
		}
	}
#endif // !UE_BUILD_SHIPPING

	while (**Stream == TEXT(' ') || **Stream == TEXT('\t'))
	{
		(*Stream)++;
	}

	int32 MatchLen = FCString::Strlen(Match);
	if (FCString::Strnicmp(*Stream, Match, MatchLen) == 0)
	{
		*Stream += MatchLen;
		if( !FChar::IsAlnum(**Stream))
//		if( !FChar::IsAlnum(**Stream) && (**Stream != '_') && (**Stream != '.'))		// more correct e.g. a cvar called "log.abc" should work but breaks some code so commented out
		{
			while (**Stream == TEXT(' ') || **Stream == TEXT('\t'))
			{
				(*Stream)++;
			}

			FCoreDelegates::OnNamedCommandParsed.Broadcast(Match);

			return true; // Success.
		}
		else
		{
			*Stream -= MatchLen;
			return false; // Only found partial match.
		}
	}
	else
	{
		return false; // No match.
	}
}

//
// Get next command.  Skips past comments and cr's.
//
void FParse::Next( const TCHAR** Stream )
{
	// Skip over spaces, tabs, cr's, and linefeeds.
	SkipJunk:
	while( **Stream==' ' || **Stream==9 || **Stream==13 || **Stream==10 )
		++*Stream;

	if( **Stream==';' )
	{
		// Skip past comments.
		while( **Stream!=0 && **Stream!=10 && **Stream!=13 )
			++*Stream;
		goto SkipJunk;
	}

	// Upon exit, *Stream either points to valid Stream or a nul.
}

static bool IsDelimiterOrWhitespace(TCHAR Character, const TCHAR SingleCharacterDelimiter)
{
	return (SingleCharacterDelimiter == TEXT('\0') && FChar::IsWhitespace(Character))
		|| (SingleCharacterDelimiter != TEXT('\0') && Character == SingleCharacterDelimiter);
}

//
// Grab the next space-delimited (or SingleCharacterDelimiter-delimited) string from the input stream.
// If quoted, gets entire quoted string.
//
bool FParse::Token(const TCHAR*& Str, TCHAR* Result, int32 MaxLen, bool bUseEscape, const TCHAR SingleCharacterDelimiter/* = TEXT('\0')*/)
{
	int32 Len=0;

	// Skip preceeding delimiters (either spaces and tabs or custom delimiters)
	while (IsDelimiterOrWhitespace(*Str, SingleCharacterDelimiter))
	{
		Str++;
	}

	if( *Str == TEXT('"') )
	{
		// Get quoted string.
		Str++;
		while (*Str && *Str != TEXT('"') && (Len + 1) < MaxLen)
		{
			TCHAR Character = *Str++;
			if (Character == TEXT('\\') && bUseEscape)
			{
				// Get escape.
				Character = *Str++;
				if (!Character)
				{
					break;
				}
			}
			if ((Len + 1) < MaxLen)
			{
				Result[Len++] = Character;
			}
		}
		if (*Str == TEXT('"'))
		{
			Str++;
		}
	}
	else
	{
		// Get unquoted string (that might contain a quoted part, which will be left intact).
		// For example, -ARG="foo bar baz", will be treated as one token, with quotes intact
		bool bInQuote = false;

		while (1)
		{
			TCHAR Character = *Str;
			if (Character == 0)
			{
				break;
			}
			if (!bInQuote)
			{
				if (IsDelimiterOrWhitespace(Character, SingleCharacterDelimiter))
				{
					// Intentionally leave trailing delimiters unchanged
					break;
				}
			}
			Str++;

			// Preserve escapes if they're in a quoted string (the check for " is in the else to let \" work as expected)
			if (Character == TEXT('\\') && bUseEscape && bInQuote)
			{
				if ((Len+1) < MaxLen)
				{
					Result[Len++] = Character;
				}

				Character = *Str;
				if (!Character)
				{
					break;
				}
				Str++;
			}
			else if (Character == TEXT('"'))
			{
				bInQuote = !bInQuote;
			}

			if( (Len+1)<MaxLen )
			{
				Result[Len++] = Character;
			}
		}
	}
	Result[Len] = TCHAR('\0');
	return Len != 0;
}

bool FParse::Token( const TCHAR*& Str, FString& Arg, bool bUseEscape, const TCHAR SingleCharacterDelimiter/* = TEXT('\0')*/)
{
	Arg.Reset();

	// Skip preceeding delimiters (either spaces and tabs or custom delimiters)
	while (IsDelimiterOrWhitespace(*Str, SingleCharacterDelimiter))
	{
		Str++;
	}

	if ( *Str == TEXT('"') )
	{
		// Get quoted string.
		Str++;
		while (*Str && *Str != TEXT('"'))
		{
			TCHAR Character = *Str++;
			if (Character == TEXT('\\') && bUseEscape)
			{
				// Get escape.
				Character = *Str++;
				if (!Character)
				{
					break;
				}
			}

			Arg += Character;
		}

		if (*Str == TEXT('"'))
		{
			Str++;
		}
	}
	else
	{
		// Get unquoted string (that might contain a quoted part, which will be left intact).
		// For example, -ARG="foo bar baz", will be treated as one token, with quotes intact
		bool bInQuote = false;

		while (1)
		{
			TCHAR Character = *Str;
			if (Character == 0)
			{
				break;
			}
			if (!bInQuote)
			{
				if (IsDelimiterOrWhitespace(Character, SingleCharacterDelimiter))
				{
					// Consume the delimiter. If it's whitespace this isn't critical since we'll consume it at the start
					// of the next call to Token() but if it's not whitespace we won't, so we better do it now.
					Str++;
					break;
				}
			}
			Str++;

			// Preserve escapes if they're in a quoted string (the check for " is in the else to let \" work as expected)
			if (Character == TEXT('\\') && bUseEscape && bInQuote)
			{
				Arg += Character;

				Character = *Str;
				if (!Character)
				{
					break;
				}
				Str++;
			}
			else if (Character == TEXT('"'))
			{
				bInQuote = !bInQuote;
			}

			Arg += Character;
		}
	}

	return Arg.Len() > 0;
}
FString FParse::Token( const TCHAR*& Str, bool UseEscape )
{
	FString Token;

	// Preallocate some memory to avoid constant reallocations.
	Token.Reserve(1023);

	FParse::Token(Str, Token, UseEscape);
	
	Token.Shrink();

	return MoveTemp(Token);
}

bool FParse::AlnumToken(const TCHAR*& Str, FString& Arg)
{
	Arg.Reset();

	// Skip preceeding spaces and tabs.
	while (FChar::IsWhitespace(*Str))
	{
		Str++;
	}

	while (FChar::IsAlnum(*Str) || *Str == TEXT('_'))
	{
		Arg += *Str;
		Str++;
	}

	return Arg.Len() > 0;
}

//
// Get a line of Stream (everything up to, but not including, CR/LF.
// Returns 0 if ok, nonzero if at end of stream and returned 0-length string.
//
bool FParse::Line(const TCHAR** Stream, TCHAR* Result, int32 MaxLen, bool bExact)
{
	bool bGotStream = false;
	bool bIsQuoted = false;
	bool bIgnore = false;

	*Result = TCHAR('\0');
	while (**Stream != TEXT('\0') && **Stream != TEXT('\n') && **Stream != TEXT('\r') && --MaxLen > 0)
	{
		// Start of comments.
		if (!bIsQuoted && !bExact && (*Stream)[0]=='/' && (*Stream)[1] == TEXT('/'))
		{
			bIgnore = true;
		}
		
		// Command chaining.
		if (!bIsQuoted && !bExact && **Stream == TEXT('|'))
		{
			break;
		}

		// Check quoting.
		bIsQuoted = bIsQuoted ^ (**Stream == TEXT('\"'));
		bGotStream = true;

		// Got stuff.
		if (!bIgnore)
		{
			*(Result++) = *((*Stream)++);
		}
		else
		{
			(*Stream)++;
		}
	}

	if (bExact)
	{
		// Eat up exactly one CR/LF.
		if (**Stream == TEXT('\r'))
		{
			(*Stream)++;
		}

		if (**Stream == TEXT('\n'))
		{
			(*Stream)++;
		}
	}
	else
	{
		// Eat up all CR/LF's.
		while (**Stream == TEXT('\n') || **Stream == TEXT('\r') || **Stream == TEXT('|'))
		{
			(*Stream)++;
		}
	}

	*Result = TEXT('\0');
	return **Stream != TEXT('\0') || bGotStream;
}

bool FParse::Line(const TCHAR** Stream, FString& Result, bool bExact)
{
	FStringView View;
	bool bReturnValue = Line(Stream, View, bExact);
	Result = View;
	return bReturnValue;
}

bool FParse::Line(const TCHAR** Stream, FStringView& Result, bool bExact)
{
	bool bGotStream = false;
	bool bIsQuoted = false;
	bool bIgnore = false;

	Result.Reset();
	const TCHAR* StartOfLine = nullptr;

	while (**Stream != TEXT('\0') && **Stream != TEXT('\n') && **Stream != TEXT('\r'))
	{
		// Start of comments.
		if (!bIsQuoted && !bExact && (*Stream)[0] == TEXT('/') && (*Stream)[1] == TEXT('/'))
		{
			bIgnore = true;
		}

		// Command chaining.
		if (!bIsQuoted && !bExact && **Stream == TEXT('|'))
		{
			break;
		}

		// Check quoting.
		bIsQuoted = bIsQuoted ^ (**Stream == TEXT('\"'));
		bGotStream = true;

		// Got stuff.
		if (!bIgnore && !StartOfLine)
		{
			StartOfLine = (*Stream)++;
		}
		else
		{
			(*Stream)++;
		}
	}

	if (StartOfLine)
	{
		Result = FStringView(StartOfLine, int32((*Stream) - StartOfLine));
	}

	if (bExact)
	{
		// Eat up exactly one CR/LF.
		if (**Stream == TEXT('\r'))
		{
			(*Stream)++;
		}
		if (**Stream == TEXT('\n'))
		{
			(*Stream)++;
		}
	}
	else
	{
		// Eat up all CR/LF's.
		while (**Stream == TEXT('\n') || **Stream == TEXT('\r') || **Stream == TEXT('|'))
		{
			(*Stream)++;
		}
	}

	return **Stream != TEXT('\0') || bGotStream;
}

template<class T>
bool ParseLineExtended(const TCHAR** InOutStream, T& Result, int32& LinesConsumed, FParse::ELineExtendedFlags Flags)
{
	const TCHAR* Stream = *InOutStream;
	bool bGotStream = false;
	bool bIsQuoted = false;
	bool bIgnore = false;
	int32 BracketDepth = 0;

	const bool bBreakOnPipe = EnumHasAnyFlags(Flags, FParse::ELineExtendedFlags::BreakOnPipe);
	const bool bHandleBracketMultiline = EnumHasAnyFlags(Flags, FParse::ELineExtendedFlags::AllowBracketedMultiline);
	const bool bHandleEscapedMultiline = EnumHasAnyFlags(Flags, FParse::ELineExtendedFlags::AllowEscapedEOLMultiline);
	const bool bHandleDoubleSlashComments = EnumHasAnyFlags(Flags, FParse::ELineExtendedFlags::SwallowDoubleSlashComments);
	const bool bHandleSemicolonComments = EnumHasAnyFlags(Flags, FParse::ELineExtendedFlags::SwallowSemicolonComments);

	Result.Reset();
	LinesConsumed = 0;

	auto IsLineBreak = [](const TCHAR* Str, bool bProcessPipeAsBreak) -> bool
		{
			return Str[0] == '\n' || Str[0] == '\r' || (bProcessPipeAsBreak && Str[0] == '|');
		};

	auto TryConsumeLineBreak = [](const TCHAR* Str, bool bProcessPipeAsBreak, int& NumChars, int& NumLines) -> bool
		{
			if (Str[0] == '\n' || Str[0] == '\r' || (bProcessPipeAsBreak && Str[0] == '|'))
			{
				NumChars = 1;
				// pipes are breaks that don't count as multiplle lines
				if (Str[0] == '|')
				{
					NumLines = 0;
				}
				else
				{
					NumLines = 1;
					// look for a \r\n (or \n\r?) pair
					if ((Str[1] == '\n' || Str[1] == '\r') && Str[0] != Str[1])
					{
						NumChars = 2;
					}
				}

				return true;
			}
			NumChars = 0;
			NumLines = 0;
			return false;
		};

	while (Stream[0] != '\0' && (!IsLineBreak(Stream, bBreakOnPipe && !bIsQuoted) || BracketDepth > 0))
	{
		// look for unquoted comments
		if (!bIsQuoted && 
			((bHandleDoubleSlashComments && Stream[0] == '/' && Stream[1] == '/') || 
			(bHandleSemicolonComments && Stream[0] == ';')))
		{
			bIgnore = true;
		}

		bGotStream = true;

		// process "allowed" line breaks (ones inside {} or after a \)
		int NumChars, NumLines;
		if ((bHandleBracketMultiline && BracketDepth > 0 && TryConsumeLineBreak(Stream, false, NumChars, NumLines)) ||
			(bHandleEscapedMultiline && Stream[0] == '\\' && TryConsumeLineBreak(Stream + 1, false, NumChars, NumLines)))
		{
			Result += TEXT(' ');
			LinesConsumed += NumLines;
			if (Stream[0] == '\\')
			{
				Stream++;
			}
			Stream += NumChars;

		}
		// check for starting or ending brace
		else if (bHandleBracketMultiline && !bIsQuoted && Stream[0] == '{')
		{
			BracketDepth++;
			Stream++;
		}
		else if (!bIsQuoted && Stream[0] == '}' && BracketDepth > 0)
		{
			BracketDepth--;
			Stream++;
		}
		// specifically consume escaped backslashes and quotes within quoted strings
		else if (bIsQuoted && !bIgnore && Stream[0] == '\\' && (Stream[1] == '\"' || Stream[1] == '\\'))
		{
			Result += FStringView(Stream, 2);
			Stream += 2;
		}
		else
		{
			bIsQuoted = bIsQuoted ^ (Stream[0] == '\"');

			// Got stuff.
			if (!bIgnore)
			{
				Result += *(Stream++);
			}
			else
			{
				Stream++;
			}
		}
	}

	if (Stream[0] == '\0')
	{
		if (bGotStream)
		{
			LinesConsumed++;
		}
	}
	else 
	{

		// start eating up line breaks (\r, \n, \r\n, maybe |)
		int NumChars, NumLines;
		while (TryConsumeLineBreak(Stream, bBreakOnPipe, NumChars, NumLines))
		{
			// move past the line break
			Stream += NumChars;
			// count lines
			LinesConsumed += NumLines;

			// if we aren't eating up extra lines, then just stop after one
			if (!EnumHasAnyFlags(Flags, FParse::ELineExtendedFlags::SwallowExtraEOLs))
			{
				break;
			}
		}
	}

	*InOutStream = Stream;
	return **InOutStream != '\0' || bGotStream;
}

bool FParse::LineExtended(const TCHAR** Stream, FString& Result, int32& LinesConsumed, ELineExtendedFlags Flags)
{
	return ParseLineExtended(Stream, Result, LinesConsumed, Flags);
}

bool FParse::LineExtended(const TCHAR** Stream, FStringBuilderBase& Result, int32& LinesConsumed, ELineExtendedFlags Flags)
{
	return ParseLineExtended(Stream, Result, LinesConsumed, Flags);
}

uint32 FParse::HexNumber(FStringView HexString)
{
	uint32 Ret = 0;

	for (const TCHAR HexChar : HexString)
	{
		Ret *= 16;
		Ret += FParse::HexDigit(HexChar);
	}

	return Ret;
}

uint64 FParse::HexNumber64(FStringView HexString)
{
	uint64 Ret = 0;

	for (const TCHAR HexChar : HexString)
	{
		Ret *= 16;
		Ret += FParse::HexDigit(HexChar);
	}

	return Ret;
}

bool FParse::SchemeNameFromURI(const TCHAR* URI, FString& OutSchemeName)
{
	for(int32 Idx = 0;;Idx++)
	{
		if(!FChar::IsAlpha(URI[Idx]) && !FChar::IsDigit(URI[Idx]) && URI[Idx] != TEXT('+') && URI[Idx] != TEXT('.') && URI[Idx] != TEXT('-'))
		{
			if(Idx > 0 && URI[Idx] == TEXT(':'))
			{
				OutSchemeName = FString::ConstructFromPtrSize(URI, Idx);
				return true;
			}
			return false;
		}
	}
}

static TCHAR GetCloseBracketCharacterForOpenBracket(TCHAR Character)
{
	switch (Character)
	{
	case TEXT('('):	return TEXT(')');
	case TEXT('['):	return TEXT(']');
	case TEXT('{'):	return TEXT('}');
	default:		return TEXT('\0'); // no match
	}
}

bool FParse::Expression(const TCHAR*& Str, FString& OutExpression, bool bUseEscape, const TCHAR SingleCharacterDelimiter/* = TEXT('\0')*/)
{
	OutExpression.Reset();

	const TCHAR* OriginalStr = Str;

	// Skip preceeding delimiters (either spaces and tabs or custom delimiters)
	while (IsDelimiterOrWhitespace(*Str, SingleCharacterDelimiter))
	{
		Str++;
	}

	if (*Str == TEXT('"'))
	{
		// Forward to Token() to get quoted string.
		return FParse::Token(Str, OutExpression, bUseEscape, SingleCharacterDelimiter);
	}

	// Get unquoted string (that might contain a quoted part, which will be left intact).
	// For example, -ARG="foo bar baz", will be treated as one token, with quotes intact
	bool bInQuote = false;
	TArray<TCHAR> BracketStack;

	while (1)
	{
		TCHAR Character = *Str;
		if (Character == 0)
		{
			break;
		}
		if (!bInQuote)
		{
			// Increase and decrease bracket level but we don't care about matching pairs.
			// This is only to isolate delimiters, not to provide syntax validation.
			if (Character == TEXT('(') || Character == TEXT('[') || Character == TEXT('{'))
			{
				BracketStack.Add(Character);
			}
			else if (Character == TEXT(')') || Character == TEXT(']') || Character == TEXT('}'))
			{
				if (BracketStack.IsEmpty())
				{
					// If bracket stack is empty, interpret closing bracket as delimiter when parsed as interleaved expression.
					// Example: "A=(B=(C,D))"
					//              ^      ^__ This bracket ends parsing the expression "B=(C,D)"
					//              |__ Start position
					break;
				}
				if (GetCloseBracketCharacterForOpenBracket(BracketStack.Top()) != Character)
				{
					// Could not match closing bracket with previous open bracket
					break;
				}
				BracketStack.Pop();
			}

			if (BracketStack.IsEmpty())
			{
				if (IsDelimiterOrWhitespace(Character, SingleCharacterDelimiter))
				{
					// Intentionally leave trailing delimiters unchanged
					break;
				}
			}
		}
		Str++;

		// Preserve escapes if they're in a quoted string (the check for " is in the else to let \" work as expected)
		if (Character == TEXT('\\') && bUseEscape && bInQuote)
		{
			OutExpression += Character;

			Character = *Str;
			if (!Character)
			{
				break;
			}
			Str++;
		}
		else if (Character == TEXT('"'))
		{
			bInQuote = !bInQuote;
		}

		OutExpression += Character;
	}

	// If brackets or quotation marks are not balanced, return false as parsing has failed and reset the input string
	if (!BracketStack.IsEmpty() || bInQuote)
	{
		Str = OriginalStr;
		OutExpression.Reset();
		return false;
	}

	return OutExpression.Len() > 0;
}


#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

#include "Misc/AutomationTest.h"
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FParseLineExtendedTest, "System.Core.Misc.ParseLineExtended", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool FParseLineExtendedTest::RunTest(const FString& Parameters)
{
	const TCHAR* Tests[] = {
		TEXT("Test string"),                            // Normal string
		TEXT("{Test string}"),                          // Braced string
		TEXT("\"Test string\""),                        // Quoted string
		TEXT("\"Test \\\"string\\\"\""),                // Quoted string w/ escaped quotes
		TEXT("a=\"Test\", b=\"Test\""),                 // Quoted value list
		TEXT("a=\"Test\\\\\", b=\"{Test}\""),           // Quoted value list w/ escaped backslash preceeding closing quote
		TEXT("a=\"Test\\\\\\\" String\", b=\"{Test}\""),// Quoted value list w/ escaped backslash preceeding escaped quote
		TEXT("Test=(Inner=\"{content}\")"),             // Nested value list
	};

	const TCHAR* Expected[] = {
		TEXT("Test string"),
		TEXT("Test string"),
		TEXT("\"Test string\""),
		TEXT("\"Test \\\"string\\\"\""),
		TEXT("a=\"Test\", b=\"Test\""),
		TEXT("a=\"Test\\\\\", b=\"{Test}\""),
		TEXT("a=\"Test\\\\\\\" String\", b=\"{Test}\""),
		TEXT("Test=(Inner=\"{content}\")"),
	};

	int32 LinesConsumed = 0;
	FString Result;

	for (int32 Index = 0; Index < UE_ARRAY_COUNT(Tests); ++Index)
	{
		LinesConsumed = 0;
		Result.Reset();

		const TCHAR* Stream = Tests[Index];
		bool bSuccess = FParse::LineExtended(&Stream, Result, LinesConsumed, FParse::ELineExtendedFlags::OldDefaultMode);
		TestTrue(*FString::Printf(TEXT("Expecting parsed line [%s] to be [%s]. Result was [%s]."), Tests[Index], Expected[Index], *Result), bSuccess && Result == Expected[Index]);
	}

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FParseExpressionSimpleTest, "System.Core.Misc.ParseExpressionSimple", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool FParseExpressionSimpleTest::RunTest(const FString& Parameters)
{
	const TCHAR* Tests[] =
	{
		TEXT("Test string"),                                    // Normal string
		TEXT("{Test string}"),                                  // Braced string
		TEXT("\"Test string\""),                                // Quoted string -> quotation marks will be trimmed
		TEXT("\"Test \\\"string\\\"\""),                        // Quoted string w/ escaped quotes
		TEXT("a=\"Test\",b=\"Test\""),                          // Quoted value list -> quotation marks are left in tact
		TEXT("a=\"Test\\\\\",b=\"{Test}\""),                    // Quoted value list w/ escaped backslash preceeding closing quote
		TEXT("a=\"Test\\\\\\\" String\",b=\"{Test}\""),         // Quoted value list w/ escaped backslash preceeding escaped quote
		TEXT("Test=(Inner=\"{content}\")"),                     // Nested value list
		TEXT("Test=(Inner=\"{content}\",Inner2=[\"{{{{\"])"),   // Double nested value list
		TEXT("B=(C,D))"),                                       // Trailing brackets when parsed from interleaved expressions, e.g. "A=(B=(C,D))"
		TEXT("(Inner=\"{content}\",Inner2=[\"{{{{\"]"),         // Failed expression due to unbalanced brackets
		TEXT(")Inner("),                                        // Failed expression due to wrong bracket orientation
		TEXT("([MismatchedBrackets)]"),                         // Failed expression due to mismatched brackets
	};

	struct FParseExpressionExpectedTestResult
	{
		bool bSuccess;
		const TCHAR Delimiter;
		TArray<const TCHAR*> ExpectedExpressions;
	};

	const FParseExpressionExpectedTestResult ExpectedResults[] =
	{
		FParseExpressionExpectedTestResult{ true, TEXT(' '), { TEXT("Test"), TEXT("string") } },
		FParseExpressionExpectedTestResult{ true, TEXT(' '), { TEXT("{Test string}") } },
		FParseExpressionExpectedTestResult{ true, TEXT(' '), { TEXT("Test string") } },
		FParseExpressionExpectedTestResult{ true, TEXT(' '), { TEXT("Test \"string\"") } },
		FParseExpressionExpectedTestResult{ true, TEXT(','), { TEXT("a=\"Test\""), TEXT("b=\"Test\"") } },
		FParseExpressionExpectedTestResult{ true, TEXT(','), { TEXT("a=\"Test\\\\\""), TEXT("b=\"{Test}\"") } },
		FParseExpressionExpectedTestResult{ true, TEXT(','), { TEXT("a=\"Test\\\\\\\" String\""), TEXT("b=\"{Test}\"") } },
		FParseExpressionExpectedTestResult{ true, TEXT('='), { TEXT("Test"), TEXT("(Inner=\"{content}\")") } },
		FParseExpressionExpectedTestResult{ true, TEXT('='), { TEXT("Test"), TEXT("(Inner=\"{content}\",Inner2=[\"{{{{\"])") } },
		FParseExpressionExpectedTestResult{ true, TEXT('='), { TEXT("B"), TEXT("(C,D)")}},
		FParseExpressionExpectedTestResult{ false, TEXT('='), { } },
		FParseExpressionExpectedTestResult{ false, TEXT('\0'), { } },
		FParseExpressionExpectedTestResult{ false, TEXT('\0'), { } },
	};

	FString Result;
	for (int32 TestIndex = 0; TestIndex < UE_ARRAY_COUNT(ExpectedResults); ++TestIndex)
	{
		const TCHAR* Stream = Tests[TestIndex];
		const FParseExpressionExpectedTestResult& Expected = ExpectedResults[TestIndex];
		if (Expected.bSuccess)
		{
			for (int32 ExpressionIndex = 0; ExpressionIndex < Expected.ExpectedExpressions.Num(); ++ExpressionIndex)
			{
				Result.Reset();
				const TCHAR* ExpectedExpression = Expected.ExpectedExpressions[ExpressionIndex];
				bool bSuccess = FParse::Expression(Stream, Result, true, Expected.Delimiter);
				TestTrue(*FString::Printf(TEXT("Expecting parsed expression [%s] %d/%d to be [%s]. Result was [%s]."),
					Tests[TestIndex], ExpressionIndex + 1, Expected.ExpectedExpressions.Num(), ExpectedExpression, *Result), bSuccess && Result == ExpectedExpression);
			}
		}
		else
		{
			Result.Reset();
			bool bSuccess = FParse::Expression(Stream, Result, true, Expected.Delimiter);
			TestFalse(*FString::Printf(TEXT("Expecting parsed expression [%s] to fail. Result was [%s]."), Tests[TestIndex], *Result), bSuccess);
		}
	}

	return true;
}

#endif //!(UE_BUILD_SHIPPING || UE_BUILD_TEST)



namespace 
{
struct FGrammarBasedParser 
{
	TFunctionRef<void(FStringView, FStringView)> OnCmd;
	const TCHAR* Cursor;
	FParse::EGrammarBasedParseFlags BehaviourFlags;

	FParse::FGrammarBasedParseResult Result;

	FGrammarBasedParser(TFunctionRef<void(FStringView, FStringView)> InOnCmd, const TCHAR* InCursor, FParse::EGrammarBasedParseFlags InBehaviourFlags) : OnCmd{ MoveTemp(InOnCmd) }, Cursor{ InCursor }, BehaviourFlags{ InBehaviourFlags }, Result{} {}
	
	const TCHAR* SkipWhitespace() 
	{
		while (FChar::IsWhitespace(*Cursor))
		{
			++Cursor;
		}
		return Cursor;
	}

	bool MatchChar(TCHAR Char)
	{
		if (*Cursor == Char)
		{
			++Cursor;
			return true;
		}
		return false;
	}

	bool MatchBetween(TCHAR Min, TCHAR Max)
	{
		if (Min <= *Cursor && *Cursor <= Max)
		{
			++Cursor;
			return true;
		}
		return false;
	}

	bool MatchValueChar()
	{
		if (!FChar::IsWhitespace(*Cursor)
			&& (*Cursor != TCHAR('"')))
		{
			++Cursor;
			return true;
		}
		return false;
	}

	bool IsAt(TCHAR Char) const
	{
		return *Cursor == Char;
	}

	bool IsEnd() const
	{
		return *Cursor == TCHAR('\0');
	}

	void SetError(FParse::EGrammarBasedParseErrorCode Code, const TCHAR* At)
	{
		Result.At = At;
		Result.ErrorCode = Code;
	}

	void SetError(FParse::EGrammarBasedParseErrorCode Code)
	{
		SetError(Code, Cursor);
	}

	bool HasError() const
	{
		return Result.ErrorCode > FParse::EGrammarBasedParseErrorCode::NotRun;
	}

	template<typename OperationType>
	void ZeroOrMore(OperationType&& ParseExpression)
	{
		for (;;) 
		{
			if (HasError() || IsEnd())
			{
				break;
			}

			if (!ParseExpression()) 
			{
				break;
			}
		}
	}

	FStringView ParseLine()
	{
		const TCHAR* Start = SkipWhitespace();
		ZeroOrMore([this]() 
		{
			FStringView ResultCmd = ParseCmd();
			return ResultCmd.Len() != 0;
		});

		if (!HasError())
		{
			SetError(FParse::EGrammarBasedParseErrorCode::Succeeded);
			return FStringView{ Start,  UE_PTRDIFF_TO_INT32(Cursor - Start) };
		}

		return {};
	}

	FStringView ParseCmd()
	{
		const TCHAR* Start = Cursor;

		if (MatchChar(TCHAR('"')))
		{
			const TCHAR* QuoteAt = Start;

			if (!(BehaviourFlags & FParse::EGrammarBasedParseFlags::AllowQuotedCommands))
			{
				SetError(FParse::EGrammarBasedParseErrorCode::DisallowedQuotedCommand, QuoteAt);
				return {};
			}
			
			Start = Cursor;
			ZeroOrMore([this]()
			{
				SkipWhitespace();
				if (IsAt(TCHAR('"')))
				{
					return false;
				}
				FStringView ResultCmd = ParseCmd();
				return ResultCmd.Len() != 0;
			});

			if (!MatchChar(TCHAR('"')))
			{
				SetError(FParse::EGrammarBasedParseErrorCode::UnBalancedQuote, QuoteAt);
				return {};
			}
			return FStringView{ Start,  UE_PTRDIFF_TO_INT32(Cursor - Start) };
		}

		FStringView Item = ParseKey();
		if (HasError())
		{
			return {};
		}
		SkipWhitespace();
		if (MatchChar(TCHAR('=')))
		{
			FStringView ItemValue = ParseValue();
			if (HasError())
			{
				return {};
			}
			OnCmd(Item, ItemValue);
		}
		else
		{
			if (Item.Len())
			{
				OnCmd(Item, FStringView{});
			}
			else
			{
				// If there is no Key then we will try consuming a value, if we can parse one
				FStringView ItemValue = ParseValue();
				if (HasError())
				{
					return {};
				}
				OnCmd(FStringView{}, ItemValue);
			}
		}
		return FStringView{ Start,  UE_PTRDIFF_TO_INT32(Cursor - Start) };
	}

	FStringView ParseKey()
	{
		const TCHAR* Start = SkipWhitespace();
		if (!MatchChar(TCHAR('/')))
		{
			MatchChar(TCHAR('-'));
			MatchChar(TCHAR('-'));
		}
		ParseIdent();
		return FStringView{ Start,  UE_PTRDIFF_TO_INT32(Cursor - Start) };
	}

	FStringView ParseValue()
	{
		const TCHAR* Start = SkipWhitespace();

		// String literal
		if (MatchChar(TCHAR('"')))
		{
			while (*Cursor && (TCHAR('"') != *Cursor))
			{
				++Cursor;
			}

			if (!MatchChar(TCHAR('"')))
			{
				SetError(FParse::EGrammarBasedParseErrorCode::UnBalancedQuote, Start);
				return {};
			}
			return FStringView{ Start,  UE_PTRDIFF_TO_INT32(Cursor - Start) };
		}

		// Some other word like value
		// or maybe a file path
		ZeroOrMore([this]()
		{
			return MatchValueChar();
		});
		return FStringView{ Start,  UE_PTRDIFF_TO_INT32(Cursor - Start) };
	}

	

	FStringView ParseIdent()
	{
		const TCHAR* Start = Cursor;
		// [_a-zA-Z]
		if (FChar::IsAlpha(*Cursor) || (*Cursor == TCHAR('_')))
		{
			// [_a-zA-Z0-9.]*
			ZeroOrMore([this]()
			{
				++Cursor;
				return FChar::IsAlnum(*Cursor) || IsAt(TCHAR('_')) || IsAt(TCHAR('.'));
			});
		}
		return FStringView{ Start,  UE_PTRDIFF_TO_INT32(Cursor - Start) };
	}

public:
	static FParse::FGrammarBasedParseResult DoParse(const TCHAR* Stream, TFunctionRef<void(FStringView, FStringView)> OnCommandCallback, FParse::EGrammarBasedParseFlags Flags)
	{
		// NOTE: if you modify this parser, please update the Grammar in the header.
		FGrammarBasedParser Parser{ OnCommandCallback, Stream, Flags };
		Parser.ParseLine();
		return MoveTemp(Parser.Result);
	}
};

}

FParse::FGrammarBasedParseResult FParse::GrammarBasedCLIParse(const TCHAR* Stream, TFunctionRef<void(FStringView, FStringView)> OnCommandCallback, EGrammarBasedParseFlags Flags)
{
	return FGrammarBasedParser::DoParse(Stream, MoveTemp(OnCommandCallback), Flags);
}
