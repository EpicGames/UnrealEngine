// Copyright Epic Games, Inc. All Rights Reserved.

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/StringFwd.h"
#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "Misc/AssertionMacros.h"
#include "Misc/NamePermissionList.h"
#include "Misc/PathViews.h"
#include "Misc/StringBuilder.h"
#include "Templates/Tuple.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

FPathPermissionList::FPathPermissionList(EPathPermissionListType InType) : ListType(InType) { }

bool FPathPermissionList::PassesFilter(const FStringView Item) const
{
	if (DenyListAll.Num() > 0)
	{
		return false;
	}

	VerifyItemMatchesListType(Item);

	if (!AllowTree.IsEmpty() || !DenyTree.IsEmpty())
	{
		if (!AllowTree.IsEmpty() && !AllowTree.Contains(Item))
		{
			// If the allow tree doesn't include the item, and
			// no delegates exist that could disable any
			// filters, then the item doesn't pass the filter.
			if (EnablePathFilterDelegates.IsEmpty())
			{
				return false;
			}

			// Check to make sure at least one filter is still enabled
			for (const TTuple<FName, FDelegateEnablePathFilter>& Entry : EnablePathFilterDelegates)
			{
				// If there is a filter that enables this permission list, and is keeping
				// it enabled, then the item doesn't pass the filter. But if a filter is
				// disabled, we can check the others.
				if (Entry.Value.IsBound() && Entry.Value.Execute())
				{
					return false;
				}
			}
		}

		// It's possible that the allow tree is empty or the filters are all disabled, so
		// see if this is an item we are explicitly denying.
		if (const FPermissionListOwners* Owners = DenyTree.Find(Item))
		{
			// If the deny tree includes the item, and  no delegates exist that would disable
			// this list, then the item doesn't pass the filter.
			if (EnablePathFilterDelegates.IsEmpty())
			{
				return false;
			}

			// At this point there are some delegates that could disable this list, so make sure
			// at least one wants to keep it on.
			for (FName Owner : *Owners)
			{
				if (const FDelegateEnablePathFilter* Filter = EnablePathFilterDelegates.Find(Owner))
				{
					// If the filter is enabled that excludes this item, then it doesn't pass the filter.
					if (Filter->IsBound() && Filter->Execute())
					{
						return false;
					}
				}
			}

			// Here, we found no enabled filters that would deny the item even though it's in the deny list.
			// At that point, we should allow the item despite it being in the deny list.
		}
	}

	return true;
}

bool FPathPermissionList::PassesFilter(const FName Item) const
{
	return PassesFilter(FNameBuilder(Item));
}

bool FPathPermissionList::PassesFilter(const TCHAR* Item) const
{
	return PassesFilter(FStringView(Item));
}

bool FPathPermissionList::PassesStartsWithFilter(const FStringView Item, const bool bAllowParentPaths) const
{
	switch (PassesStartsWithFilterRecursive(Item, bAllowParentPaths))
	{
		case EPathPermissionPrefixResult::Fail:
		case EPathPermissionPrefixResult::FailRecursive:
			return false;
		case EPathPermissionPrefixResult::Pass:
		case EPathPermissionPrefixResult::PassRecursive:
		default:
			return true;
	}
}

bool FPathPermissionList::PassesStartsWithFilter(const FName Item, const bool bAllowParentPaths) const
{
	return PassesStartsWithFilter(FNameBuilder(Item), bAllowParentPaths);
}

bool FPathPermissionList::PassesStartsWithFilter(const TCHAR* Item, const bool bAllowParentPaths) const
{
	return PassesStartsWithFilter(FStringView(Item), bAllowParentPaths);
}

EPathPermissionPrefixResult FPathPermissionList::PassesStartsWithFilterRecursive(const FStringView Item,
	const bool bAllowParentPaths) const
{
	VerifyItemMatchesListType(Item);

	if (!HasFiltering())
	{
		return EPathPermissionPrefixResult::PassRecursive;
	}

	if (DenyListAll.Num() > 0)
	{
		return EPathPermissionPrefixResult::FailRecursive;
	}

	bool bChildrenMayBeDenied = false;
	if (!DenyTree.IsEmpty())
	{
		if (DenyTree.FindClosestValue(Item) != nullptr)
		{
			return EPathPermissionPrefixResult::FailRecursive;
		}
	}

	if (AllowTree.IsEmpty())
	{
		// Return value here is dependent on whether child paths might still fail deny lists
		return DenyTree.ContainsChildPaths(Item) ? EPathPermissionPrefixResult::Pass
												 : EPathPermissionPrefixResult::PassRecursive;
	}

	bool bPassedAllowList = AllowTree.FindClosestValue(Item) != nullptr;
	if (!bPassedAllowList && bAllowParentPaths && AllowTree.ContainsChildPaths(Item))
	{
		bPassedAllowList = true;
	}
	if (bPassedAllowList)
	{
		// If we pass an allow list entry, we might later fail a deny list entry
		// This logic is also correct if bAllowParent paths is true
		// 	- child paths of Item will also be parents of an entry in AllowTree
		//	- child paths of Item may also be present in DenyTree
		return DenyTree.ContainsChildPaths(Item) ? EPathPermissionPrefixResult::Pass
												 : EPathPermissionPrefixResult::PassRecursive;
	}

	// If we don't match any allow list entries now, check if we might later pass a longer allow list entry
	// This logic is also correct if bAllowParentPaths is true - if there were a parent path of Item in the tree
	// it would have matched above and covered future calls with children of Item
	return AllowTree.ContainsChildPaths(Item) ? EPathPermissionPrefixResult::Fail
											  : EPathPermissionPrefixResult::FailRecursive;
}

bool FPathPermissionList::ContainsDenyListItem(FStringView Item) const
{
	return DenyTree.Contains(Item);
}

bool FPathPermissionList::AddDenyListItem(const FName OwnerName, const FStringView Item)
{
	VerifyItemMatchesListType(Item);

	bool bExisted = false;
	DenyTree.FindOrAdd(Item, &bExisted).AddUnique(OwnerName);
	bool bFilterChanged = !bExisted;
	if (bFilterChanged && !bSuppressOnFilterChanged)
	{
		OnFilterChanged().Broadcast();
	}

	return bFilterChanged;
}

bool FPathPermissionList::AddDenyListItem(const FName OwnerName, const FName Item)
{
	return AddDenyListItem(OwnerName, FNameBuilder(Item));
}

bool FPathPermissionList::AddDenyListItem(const FName OwnerName, const TCHAR* Item)
{
	return AddDenyListItem(OwnerName, FStringView(Item));
}

bool FPathPermissionList::RemoveDenyListItem(const FName OwnerName, const FStringView Item)
{
	FPermissionListOwners* Owners = DenyTree.Find(Item);
	if (Owners && Owners->Remove(OwnerName) == 1)
	{
		if (Owners->Num() == 0)
		{
			DenyTree.Remove(Item);
			if (!bSuppressOnFilterChanged)
			{
				OnFilterChanged().Broadcast();
			}
			return true;
		}
	}

	return false;
}

bool FPathPermissionList::RemoveDenyListItem(const FName OwnerName, const FName Item)
{
	return RemoveDenyListItem(OwnerName, FNameBuilder(Item));
}

bool FPathPermissionList::RemoveDenyListItem(const FName OwnerName, const TCHAR* Item)
{
	return RemoveDenyListItem(OwnerName, FStringView(Item));
}

bool FPathPermissionList::HasDenyListEntries() const
{
	return DenyTree.IsEmpty() == false;
}

TArray<FString> FPathPermissionList::GetDenyListEntries() const
{
	TArray<FString> Entries;
	DenyTree.TryGetChildren({}, Entries, EDirectoryTreeGetFlags::Recursive | EDirectoryTreeGetFlags::ImpliedParent);
	return MoveTemp(Entries);
}

FPermissionListOwners FPathPermissionList::RemoveDenyListItemAndGetOwners(FStringView Item)
{
	if (FPermissionListOwners* Owners = DenyTree.Find(Item))
	{
		FPermissionListOwners RemovedOwners = MoveTemp(*Owners);
		DenyTree.Remove(Item);

		if (!bSuppressOnFilterChanged)
		{
			OnFilterChanged().Broadcast();
		}
		return RemovedOwners;
	}
	return {};
}

bool FPathPermissionList::HasAllowListEntries() const
{
	return AllowTree.IsEmpty() == false;
}

TArray<FString> FPathPermissionList::GetAllowListEntries() const
{
	TArray<FString> Entries;
	AllowTree.TryGetChildren({}, Entries, EDirectoryTreeGetFlags::Recursive | EDirectoryTreeGetFlags::ImpliedParent);
	return MoveTemp(Entries);
}

bool FPathPermissionList::AddAllowListItem(const FName OwnerName, const FStringView Item)
{
	VerifyItemMatchesListType(Item);
	bool bExisted = false;
	AllowTree.FindOrAdd(Item, &bExisted).AddUnique(OwnerName);
	bool bFilterChanged = !bExisted;
	if (bFilterChanged && !bSuppressOnFilterChanged)
	{
		OnFilterChanged().Broadcast();
	}

	return bFilterChanged;
}

bool FPathPermissionList::AddAllowListItem(const FName OwnerName, const FName Item)
{
	return AddAllowListItem(OwnerName, FNameBuilder(Item));
}

bool FPathPermissionList::AddAllowListItem(const FName OwnerName, const TCHAR* Item)
{
	return AddAllowListItem(OwnerName, FStringView(Item));
}

bool FPathPermissionList::AddDenyListAll(const FName OwnerName)
{
	const int32 OldNum = DenyListAll.Num();
	DenyListAll.AddUnique(OwnerName);

	const bool bFilterChanged = OldNum != DenyListAll.Num();
	if (bFilterChanged && !bSuppressOnFilterChanged)
	{
		OnFilterChanged().Broadcast();
	}

	return bFilterChanged;
}

bool FPathPermissionList::RemoveAllowListItem(const FName OwnerName, const FStringView Item)
{
	FPermissionListOwners* Owners = AllowTree.Find(Item);
	if (Owners && Owners->Remove(OwnerName) == 1)
	{
		if (Owners->Num() == 0)
		{
			AllowTree.Remove(Item);
			if (!bSuppressOnFilterChanged)
			{
				OnFilterChanged().Broadcast();
			}
			return true;
		}
	}

	return false;
}

bool FPathPermissionList::RemoveAllowListItem(const FName OwnerName, const FName Item)
{
	return RemoveAllowListItem(OwnerName, FNameBuilder(Item));
}

bool FPathPermissionList::RemoveAllowListItem(const FName OwnerName, const TCHAR* Item)
{
	return RemoveAllowListItem(OwnerName, FStringView(Item));
}

bool FPathPermissionList::HasFiltering() const
{
	return !DenyTree.IsEmpty() || !AllowTree.IsEmpty() || DenyListAll.Num() > 0;
}

TArray<FName> FPathPermissionList::GetOwnerNames() const
{
	TArray<FName> OwnerNames;

	TArray<FString> Entries;
	DenyTree.TryGetChildren({}, Entries, EDirectoryTreeGetFlags::Recursive | EDirectoryTreeGetFlags::ImpliedParent);
	for (const FString& DenyEntry : Entries)
	{
		if (const FPermissionListOwners* Owners = DenyTree.Find(DenyEntry))
		{
			for (FName OwnerName : *Owners)
			{
				OwnerNames.AddUnique(OwnerName);
			}
		}
	}

	Entries.Reset();
	AllowTree.TryGetChildren({}, Entries, EDirectoryTreeGetFlags::Recursive | EDirectoryTreeGetFlags::ImpliedParent);
	for (const FString& AllowEntry : Entries)
	{
		for (FName OwnerName : *AllowTree.Find(AllowEntry))
		{
			OwnerNames.AddUnique(OwnerName);
		}
	}

	for (const auto& OwnerName : DenyListAll)
	{
		OwnerNames.AddUnique(OwnerName);
	}

	return OwnerNames;
}

bool FPathPermissionList::UnregisterOwner(const FName OwnerName)
{
	bool bFilterChanged = false;

	TArray<FString> Entries;
	DenyTree.TryGetChildren({}, Entries, EDirectoryTreeGetFlags::Recursive | EDirectoryTreeGetFlags::ImpliedParent);
	for (const FString& DenyEntry : Entries)
	{
		if (FPermissionListOwners* Owners = DenyTree.Find(DenyEntry))
		{
			Owners->Remove(OwnerName);
			if (Owners->Num() == 0)
			{
				DenyTree.Remove(DenyEntry);
				bFilterChanged = true;
			}
		}
	}

	Entries.Reset();
	AllowTree.TryGetChildren({}, Entries, EDirectoryTreeGetFlags::Recursive | EDirectoryTreeGetFlags::ImpliedParent);
	for (const FString& AllowEntry : Entries)
	{
		if (FPermissionListOwners* Owners = AllowTree.Find(AllowEntry))
		{
			Owners->Remove(OwnerName);
			if (Owners->Num() == 0)
			{
				AllowTree.Remove(AllowEntry);
				bFilterChanged = true;
			}
		}
	}

	bFilterChanged |= (DenyListAll.Remove(OwnerName) > 0);

	{
		TGuardValue<bool> Guard(bSuppressOnFilterChanged, true);
		bFilterChanged |= RemoveEnablePathFilterDelegate(OwnerName);
	}

	if (bFilterChanged && !bSuppressOnFilterChanged)
	{
		OnFilterChanged().Broadcast();
	}

	return bFilterChanged;
}

bool FPathPermissionList::UnregisterOwners(const TArray<FName>& OwnerNames)
{
	bool bFilterChanged = false;
	{
		TGuardValue<bool> Guard(bSuppressOnFilterChanged, true);

		for (const FName& OwnerName : OwnerNames)
		{
			bFilterChanged |= UnregisterOwner(OwnerName);
		}
	}

	if (bFilterChanged && !bSuppressOnFilterChanged)
	{
		OnFilterChanged().Broadcast();
	}

	return bFilterChanged;
}

bool FPathPermissionList::Append(const FPathPermissionList& Other)
{
	ensureAlwaysMsgf(ListType == Other.ListType, TEXT("Trying to combine PathPermissionLists of different types"));

	bool bFilterChanged = false;
	{
		TGuardValue<bool> Guard(bSuppressOnFilterChanged, true);

		TArray<FString> Entries;
		Other.DenyTree.TryGetChildren({},
			Entries,
			EDirectoryTreeGetFlags::Recursive | EDirectoryTreeGetFlags::ImpliedParent);
		for (const FString& DenyEntry : Entries)
		{
			for (FName OwnerName : *Other.DenyTree.Find(DenyEntry))
			{
				bFilterChanged |= AddDenyListItem(OwnerName, DenyEntry);
			}
		}

		Entries.Reset();
		Other.AllowTree.TryGetChildren({},
			Entries,
			EDirectoryTreeGetFlags::Recursive | EDirectoryTreeGetFlags::ImpliedParent);
		for (const FString& AllowEntry : Entries)
		{
			for (const auto& OwnerName : *Other.AllowTree.Find(AllowEntry))
			{
				bFilterChanged |= AddAllowListItem(OwnerName, AllowEntry);
			}
		}

		for (FName OwnerName : Other.DenyListAll)
		{
			bFilterChanged |= AddDenyListAll(OwnerName);
		}
	}

	if (bFilterChanged && !bSuppressOnFilterChanged)
	{
		OnFilterChanged().Broadcast();
	}

	return bFilterChanged;
}

FPathPermissionList FPathPermissionList::CombinePathFilters(const FPathPermissionList& OtherFilter) const
{
	FPathPermissionList Result;

	Result.DenyListAll.Append(DenyListAll);
	Result.DenyListAll.Append(OtherFilter.DenyListAll);

	TArray<FString> Entries;
	DenyTree.TryGetChildren({}, Entries, EDirectoryTreeGetFlags::Recursive | EDirectoryTreeGetFlags::ImpliedParent);
	for (const FString& DenyEntry : Entries)
	{
		for (FName OwnerName : *DenyTree.Find(DenyEntry))
		{
			Result.AddDenyListItem(OwnerName, DenyEntry);
		}
	}

	Entries.Reset();
	OtherFilter.DenyTree.TryGetChildren({},
		Entries,
		EDirectoryTreeGetFlags::Recursive | EDirectoryTreeGetFlags::ImpliedParent);
	for (const FString& DenyEntry : Entries)
	{
		for (const FName& OwnerName : *OtherFilter.DenyTree.Find(DenyEntry))
		{
			Result.AddDenyListItem(OwnerName, DenyEntry);
		}
	}

	if (!AllowTree.IsEmpty() || !OtherFilter.AllowTree.IsEmpty())
	{
		Entries.Reset();
		AllowTree.TryGetChildren({},
			Entries,
			EDirectoryTreeGetFlags::Recursive | EDirectoryTreeGetFlags::ImpliedParent);
		for (const FString& AllowEntry : Entries)
		{
			if (OtherFilter.PassesStartsWithFilter(AllowEntry, true))
			{
				for (const FName& OwnerName : *AllowTree.Find(AllowEntry))
				{
					Result.AddAllowListItem(OwnerName, AllowEntry);
				}
			}
		}

		Entries.Reset();
		OtherFilter.AllowTree.TryGetChildren({},
			Entries,
			EDirectoryTreeGetFlags::Recursive | EDirectoryTreeGetFlags::ImpliedParent);
		for (const FString& AllowEntry : Entries)
		{
			if (PassesStartsWithFilter(AllowEntry, true))
			{
				for (const FName& OwnerName : *OtherFilter.AllowTree.Find(AllowEntry))
				{
					Result.AddAllowListItem(OwnerName, AllowEntry);
				}
			}
		}

		// Block everything if none of the AllowList paths passed
		if (Result.AllowTree.IsEmpty())
		{
			Result.AddDenyListAll(NAME_None);
		}
	}

	return Result;
}

bool FPathPermissionList::UnregisterOwnersAndAppend(const TArray<FName>& OwnerNamesToRemove, const FPathPermissionList& FiltersToAdd)
{
	bool bFilterChanged = false;
	{
		TGuardValue<bool> Guard(bSuppressOnFilterChanged, true);

		bFilterChanged |= UnregisterOwners(OwnerNamesToRemove);
		bFilterChanged |= Append(FiltersToAdd);
	}

	if (bFilterChanged && !bSuppressOnFilterChanged)
	{
		OnFilterChanged().Broadcast();
	}

	return bFilterChanged;
}

// Extracted the ensure condition into a separate function so that logs are easier to read
FORCEINLINE static bool IsClassPathNameOrNone(const FStringView Item)
{
	return !Item.Len() || Item[0] == '/' || Item == TEXTVIEW("None");
}

void FPathPermissionList::VerifyItemMatchesListType(const FStringView Item) const
{
	if (ListType == EPathPermissionListType::ClassPaths)
	{
		// Long names always have / as first character
		ensureAlwaysMsgf(IsClassPathNameOrNone(Item), TEXT("Short class name \"%.*s\" provided for PathPermissionList representing class paths"), Item.Len(), Item.GetData());
	}
}

FString FPathPermissionList::ToString() const
{
	TStringBuilder<4096> StringBuilder;

	auto SortAndAppendOwners = [&StringBuilder](FPermissionListOwners Owners) {
		Owners.Sort(FNameLexicalLess());

		StringBuilder.AppendChar(TCHAR('('));
		bool bFirst = true;
		for (FName Owner : Owners)
		{
			if (bFirst)
			{
				bFirst = false;
			}
			else
			{
				StringBuilder.Append(TEXT(", "));
			}
			StringBuilder.Append(Owner.ToString());
		}
		StringBuilder.AppendChar(TCHAR(')'));
	};

	if (!DenyListAll.IsEmpty())
	{
		StringBuilder.Append(TEXT("Deny All "));
		SortAndAppendOwners(DenyListAll);
		StringBuilder.Append(TEXT("\n"));;
	}

	auto AppendList = [&StringBuilder, &SortAndAppendOwners](const TDirectoryTree<FPermissionListOwners>& Tree) {
		TArray<FString> AllPaths;
		Tree.TryGetChildren({}, AllPaths, EDirectoryTreeGetFlags::Recursive | EDirectoryTreeGetFlags::ImpliedParent);
		Algo::Sort(AllPaths);
		for (const FString& Path : AllPaths)
		{
			StringBuilder.AppendChar(TCHAR('\t'));
			StringBuilder.AppendChar(TCHAR('"'));
			StringBuilder.Append(Path);
			StringBuilder.AppendChar(TCHAR('"'));
			StringBuilder.AppendChar(TCHAR(' '));

			if (const FPermissionListOwners* List = Tree.Find(Path))
			{
				SortAndAppendOwners(*List);
			}
			StringBuilder.AppendChar(TCHAR('\n'));
		}
	};

	if (!DenyTree.IsEmpty())
	{
		StringBuilder.Append(TEXT("Deny List\n"));
		AppendList(DenyTree);
	}

	if (!AllowTree.IsEmpty())
	{
		StringBuilder.Append(TEXT("Allow List\n"));
		AppendList(AllowTree);
	}

	return FString(StringBuilder);
}

void FPathPermissionList::AddEnablePathFilterDelegate(FName Owner, FDelegateEnablePathFilter Delegate)
{
	EnablePathFilterDelegates.Add(Owner, Delegate);

	if (!bSuppressOnFilterChanged)
	{
		OnFilterChanged().Broadcast();
	}
}

bool FPathPermissionList::RemoveEnablePathFilterDelegate(FName Owner)
{
	bool bRemovedDelegate = EnablePathFilterDelegates.Remove(Owner) > 0;

	if (bRemovedDelegate && !bSuppressOnFilterChanged)
	{
		OnFilterChanged().Broadcast();
	}

	return bRemovedDelegate;
}