// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/PathViews.h"

#include "Algo/Find.h"
#include "Algo/FindLast.h"
#include "Algo/Replace.h"
#include "Containers/ArrayView.h"
#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include "HAL/PlatformMisc.h"
#include "HAL/PlatformProcess.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/AsciiSet.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CString.h"
#include "Misc/Char.h"
#include "Misc/PathViewsPrivate.h"
#include "Misc/StringBuilder.h"
#include "String/Find.h"
#include "String/ParseTokens.h"
#include "Templates/Function.h"

namespace UE4PathViews_Private
{
	static bool IsSlashOrBackslash(TCHAR C) { return C == TEXT('/') || C == TEXT('\\'); }
	static bool IsNotSlashOrBackslash(TCHAR C) { return C != TEXT('/') && C != TEXT('\\'); }

	static bool IsSlashOrBackslashOrPeriod(TCHAR C) { return C == TEXT('/') || C == TEXT('\\') || C == TEXT('.'); }

	static bool StringEqualsIgnoreCaseIgnoreSeparator(FStringView A, FStringView B)
	{
		if (A.Len() != B.Len())
		{
			return false;
		}
		const TCHAR* AData = A.GetData();
		const TCHAR* AEnd = AData + A.Len();
		const TCHAR* BData = B.GetData();
		for (; AData < AEnd; ++AData, ++BData)
		{
			TCHAR AChar = FChar::ToUpper(*AData);
			TCHAR BChar = FChar::ToUpper(*BData);
			if (IsSlashOrBackslash(AChar))
			{
				if (!IsSlashOrBackslash(BChar))
				{
					return false;
				}
			}
			else
			{
				if (AChar != BChar)
				{
					return false;
				}
			}
		}
		return true;
	}

	static bool StringLessIgnoreCaseIgnoreSeparator(FStringView A, FStringView B)
	{
		const TCHAR* AData = A.GetData();
		const TCHAR* AEnd = AData + A.Len();
		const TCHAR* BData = B.GetData();
		const TCHAR* BEnd = BData + B.Len();
		for (; AData < AEnd && BData < BEnd; ++AData, ++BData)
		{
			TCHAR AChar = FChar::ToUpper(*AData);
			TCHAR BChar = FChar::ToUpper(*BData);
			if (IsSlashOrBackslash(AChar))
			{
				if (!IsSlashOrBackslash(BChar))
				{
					return true;
				}
			}
			else if (IsSlashOrBackslash(BChar))
			{
				return false;
			}
			else
			{
				if (AChar != BChar)
				{
					return AChar < BChar;
				}
			}
		}
		if (BData < BEnd)
		{
			// B is longer than A, so A is a prefix of B. A string is greater than any of its prefixes
			return true;
		}
		return false;
	}
}

FStringView FPathViews::GetCleanFilename(const FStringView& InPath)
{
	if (const TCHAR* StartPos = Algo::FindLastByPredicate(InPath, UE4PathViews_Private::IsSlashOrBackslash))
	{
		return InPath.RightChop(UE_PTRDIFF_TO_INT32(StartPos - InPath.GetData() + 1));
	}
	return InPath;
}

FStringView FPathViews::GetBaseFilename(const FStringView& InPath)
{
	const FStringView CleanPath = GetCleanFilename(InPath);
	return CleanPath.LeftChop(GetExtension(CleanPath, /*bIncludeDot*/ true).Len());
}

FStringView FPathViews::GetBaseFilenameWithPath(const FStringView& InPath)
{
	return InPath.LeftChop(GetExtension(InPath, /*bIncludeDot*/ true).Len());
}

FStringView FPathViews::GetPath(const FStringView& InPath)
{
	if (const TCHAR* EndPos = Algo::FindLastByPredicate(InPath, UE4PathViews_Private::IsSlashOrBackslash))
	{
		return InPath.Left(UE_PTRDIFF_TO_INT32(EndPos - InPath.GetData()));
	}
	return FStringView();
}

FStringView FPathViews::GetExtension(const FStringView& InPath, UE::Paths::EFlags Flags)
{
	using namespace UE::Paths;

	FStringView CleanFilename = GetCleanFilename(InPath);
	const TCHAR* Dot = !EnumHasAnyFlags(Flags, EFlags::AllowCompoundExtension)
		? Algo::FindLast(CleanFilename, TEXT('.'))
		: Algo::Find(CleanFilename, TEXT('.'));
	if (Dot)
	{
		const TCHAR* Extension = EnumHasAnyFlags(Flags, EFlags::IncludeDot) ? Dot : Dot + 1;
		return FStringView(Extension, UE_PTRDIFF_TO_INT32(InPath.GetData() + InPath.Len() - Extension));
	}
	return FStringView();
}

FStringView FPathViews::GetExtension(const FStringView& InPath, bool bIncludeDot)
{
	using namespace UE::Paths;
	return GetExtension(InPath, bIncludeDot ? EFlags::IncludeDot : EFlags::None);
}

FStringView FPathViews::GetPathLeaf(const FStringView& InPath)
{
	if (const TCHAR* EndPos = Algo::FindLastByPredicate(InPath, UE4PathViews_Private::IsNotSlashOrBackslash))
	{
		++EndPos;
		return GetCleanFilename(InPath.Left(UE_PTRDIFF_TO_INT32(EndPos - InPath.GetData())));
	}
	return FStringView();
}

bool FPathViews::HasRedundantTerminatingSeparator(FStringView A)
{
	using namespace UE4PathViews_Private;

	if (A.Len() <= 2)
	{
		if (A.Len() <= 1)
		{
			// "", "c", or "/", All of these are either not slash terminating or not redundant
			return false;
		}
		else if (!IsSlashOrBackslash(A[1]))
		{
			// "/c" or "cd"
			return false;
		}
		else if (IsSlashOrBackslash(A[0]))
		{
			// "//"
			return false;
		}
		else if (A[0] == ':')
		{
			// ":/", which is an invalid path, and we arbitrarily decide its terminating slash is not redundant
			return false;
		}
		else
		{
			// "c/"
			return true;
		}
	}
	else if (!IsSlashOrBackslash(A[A.Len() - 1]))
	{
		// "/Some/Path"
		return false;
	}
	else if (A[A.Len() - 2] == ':')
	{
		// "Volume:/",  "Volume:/Some/Path:/"
		// The first case is the root dir of the volume and is not redundant
		// The second case is an invalid path (at most one colon), and we arbitrarily decide its terminating slash is not redundant
		return false;
	}
	else
	{
		// /Some/Path/
		return true;
	}
}

bool FPathViews::IsPathLeaf(FStringView InPath)
{
	using namespace UE4PathViews_Private;

	const TCHAR* FirstSlash = Algo::FindByPredicate(InPath, IsSlashOrBackslash);
	if (FirstSlash == nullptr)
	{
		return true;
	}
	if (Algo::FindByPredicate(InPath.RightChop(UE_PTRDIFF_TO_INT32(FirstSlash - InPath.GetData())), IsNotSlashOrBackslash) == nullptr)
	{
		// The first slash is after the last non-slash
		// This means it is either required for e.g. // or D:/ or /, or it is a redundant terminating slash D:/Foo/
		// In all of those cases the token is still considered a (possibly unnormalized) leaf
		return true;
	}
	return false;

}

void FPathViews::IterateComponents(FStringView InPath, TFunctionRef<void(FStringView)> ComponentVisitor)
{
	UE::String::ParseTokensMultiple(InPath, { '/', '\\' }, ComponentVisitor);
}

void FPathViews::IterateAncestors(FStringView InPath, TFunctionRef<bool(FStringView)> AncestorVisitor)
{
	constexpr const FAsciiSet Separators("\\/");
	constexpr const FAsciiSet NonSeparators(~Separators);

	// First check for a root specifier such as C:\, /Root, //ShareName
	// When we reach this root we stop searching for path separators and yield it as the final ancestor.
	FStringView Root;
	FStringView Remainder;
	FPathViews::SplitVolumeSpecifier(InPath, Root, Remainder);
	if (Root.Len() > 0 && Root[Root.Len() - 1] == ':' && Remainder.Len() > 0 && Separators.Contains(Remainder[0]))
	{
		// SplitVolumeSpecifier returns Drive: without separator but it's more convenient to yield Drive:\ for use as a path in other APIs
		Root = { Root.GetData(), Root.Len() + 1 };
	}
	else if (Root.Len() == 0 && InPath.Len() > 0 && Separators.Contains(InPath[0]))
	{
		// SplitVolumeSpecifier does not return a linux/package path-style root of / - split it out explicitly
		Root = { InPath.GetData(), 1 };
	}

	do
	{
		// Strip any trailing path separators
		InPath = FAsciiSet::TrimSuffixWith(InPath, Separators);

		// If we've iterated into the root, yield that as the last element
		// Need to use pointer checks rather than length here because we may have stripped extra separators from the root
		if (InPath.GetData() + InPath.Len() <= Root.GetData() + Root.Len())
		{
			break;
		}

		if (!AncestorVisitor(InPath))
		{
			return;
		}

		// Find next separator
		InPath = FAsciiSet::TrimSuffixWithout(InPath, Separators);
	}
	while (!InPath.IsEmpty());

	if (!Root.IsEmpty())
	{
		AncestorVisitor(Root);
	}
}

void FPathViews::Split(const FStringView& InPath, FStringView& OutPath, FStringView& OutName, FStringView& OutExt)
{
	const FStringView CleanName = GetCleanFilename(InPath);
	const TCHAR* DotPos = Algo::FindLast(CleanName, TEXT('.'));
	const int32 NameLen = DotPos ? UE_PTRDIFF_TO_INT32(DotPos - CleanName.GetData()) : CleanName.Len();
	OutPath = InPath.LeftChop(CleanName.Len() + 1);
	// If there is a ., check for CleanName == ..; that is a special case that is incorrectly handled if we
	// always interpret the last . as a file extension marker
	if (DotPos && CleanName == TEXTVIEW(".."))
	{
		OutName = CleanName;
		OutExt.Reset();
	}
	else
	{
		OutName = CleanName.Left(NameLen);
		OutExt = CleanName.RightChop(NameLen + 1);
	}
}

FString FPathViews::ChangeExtension(const FStringView& InPath, const FStringView& InNewExtension)
{
	// Make sure the period we found was actually for a file extension and not part of the file path.
	const TCHAR* PathEndPos = Algo::FindLastByPredicate(InPath, UE4PathViews_Private::IsSlashOrBackslashOrPeriod);
	if (PathEndPos != nullptr && *PathEndPos == TEXT('.'))
	{
		const int32 Pos = UE_PTRDIFF_TO_INT32(PathEndPos - InPath.GetData());
		const FStringView FileWithoutExtension = InPath.Left(Pos);

		if (!InNewExtension.IsEmpty() && !InNewExtension.StartsWith(TEXT('.')))
		{
			// The new extension lacks a period so we need to add it ourselves.
			FString Result(FileWithoutExtension, InNewExtension.Len() + 1);
			Result += '.';
			Result += InNewExtension;

			return Result;
		}
		else
		{
			FString Result(FileWithoutExtension, InNewExtension.Len());
			Result += InNewExtension;

			return Result;
		}
	}
	else
	{
		return FString(InPath);
	}
}

FString FPathViews::SetExtension(const FStringView& InPath, const FStringView& InNewExtension)
{
	int32 Pos = INDEX_NONE;
	const TCHAR* PathEndPos = Algo::FindLastByPredicate(InPath, UE4PathViews_Private::IsSlashOrBackslashOrPeriod);
	if (PathEndPos != nullptr && *PathEndPos == TEXT('.'))
	{
		Pos = UE_PTRDIFF_TO_INT32(PathEndPos - InPath.GetData());
	}

	const FStringView FileWithoutExtension = Pos == INDEX_NONE ? InPath : InPath.Left(Pos);

	if (!InNewExtension.IsEmpty() && !InNewExtension.StartsWith(TEXT('.')))
	{
		// The new extension lacks a period so we need to add it ourselves.
		FString Result(FileWithoutExtension, InNewExtension.Len() + 1);
		Result += TEXT('.');
		Result += InNewExtension;

		return Result;
	}
	else
	{
		FString Result(FileWithoutExtension, InNewExtension.Len());
		Result += InNewExtension;

		return Result;
	}
}

bool FPathViews::IsSeparator(TCHAR c)
{
	return UE4PathViews_Private::IsSlashOrBackslash(c);
}

bool FPathViews::Equals(FStringView A, FStringView B)
{
	using namespace UE4PathViews_Private;

	while (HasRedundantTerminatingSeparator(A))
	{
		A.LeftChopInline(1);
	}
	while (HasRedundantTerminatingSeparator(B))
	{
		B.LeftChopInline(1);
	}
	return StringEqualsIgnoreCaseIgnoreSeparator(A, B);
}

bool FPathViews::Less(FStringView A, FStringView B)
{
	using namespace UE4PathViews_Private;

	while (HasRedundantTerminatingSeparator(A))
	{
		A.LeftChopInline(1);
	}
	while (HasRedundantTerminatingSeparator(B))
	{
		B.LeftChopInline(1);
	}
	return StringLessIgnoreCaseIgnoreSeparator(A, B);
}

bool FPathViews::TryMakeChildPathRelativeTo(FStringView Child, FStringView Parent, FStringView& OutRelPath)
{
	using namespace UE4PathViews_Private;

	while (HasRedundantTerminatingSeparator(Parent))
	{
		Parent.LeftChopInline(1);
		// Note that Parent can not be the empty string, because HasRedundantTerminatingSeparator("/") is false
	}
	if (Parent.Len() == 0)
	{
		// We arbitrarily define an empty string as never the parent of any directory
		OutRelPath.Reset();
		return false;
	}
	if (Child.Len() < Parent.Len())
	{
		// A shorter path can not be a child path
		OutRelPath.Reset();
		return false;
	}

	check(Child.Len() > 0 && Parent.Len() > 0);

	if (!StringEqualsIgnoreCaseIgnoreSeparator(Parent, Child.SubStr(0, Parent.Len())))
	{
		// Child paths match their parent paths exactly up to the parent's length
		OutRelPath.Reset();
		return false;
	}
	if (Child.Len() == Parent.Len())
	{
		// The child is equal to the parent, which we define as being a child path
		OutRelPath.Reset();
		return true;
	}

	check(Child.Len() >= Parent.Len() + 1);

	if (IsSlashOrBackslash(Parent[Parent.Len() - 1]))
	{
		// Parent is the special case "//", "Volume:/", or "/" (these are the only cases that end in / after HasRedundantTerminatingSeparator)
		if (Parent.Len() == 1 && IsSlashOrBackslash(Child[1]))
		{
			// Parent is "/", Child is "//SomethingOrNothing". // is not a child of /, so this is not a child
			OutRelPath.Reset();
			return false;
		}
		// Since child starts with parent, including the terminating /, child is a child path of parent
		OutRelPath = Child.RightChop(Parent.Len()); // Chop all of the parent including the terminating slash
	}
	else
	{
		if (!IsSlashOrBackslash(Child[Parent.Len()]))
		{
			// Child is in a different directory that happens to have the parent's final directory component as a prefix
			// e.g. "Volume:/Some/Path", "Volume:/Some/PathOther"
			OutRelPath.Reset();
			return false;
		}
		// Parent   = Volume:/Some/Path
		// Child    = Volume:/Some/Path/Child1/Child2/etc
		// or Child = Volume:/Some/Path/
		// or Child = Volume:/Some/Path/////
		OutRelPath = Child.RightChop(Parent.Len() + 1); // Chop the Parent and the slash in the child that follows the parent
	}
	// If there were invalid duplicate slashes after the parent path, remove them since starting with a slash
	// is not a valid relpath
	while (OutRelPath.Len() > 0 && IsSlashOrBackslash(OutRelPath[0]))
	{
		OutRelPath.RightChopInline(1);
	}
	return true;
}

bool FPathViews::IsParentPathOf(FStringView Parent, FStringView Child)
{
	FStringView RelPath;
	return TryMakeChildPathRelativeTo(Child, Parent, RelPath);
}

bool FPathViews::IsRelativePath(FStringView InPath)
{
	using namespace UE4PathViews_Private;

	if (const TCHAR* EndPos = Algo::FindByPredicate(InPath, IsSlashOrBackslash))
	{
		const int32 FirstLen = UE_PTRDIFF_TO_INT32(EndPos - InPath.GetData());
		if (FirstLen == 0)
		{
			// Path starts with /; it may be either /Foo or //Foo
			return false;
		}
		else if (InPath[FirstLen - 1] == TEXT(':'))
		{
			// InPath == Volume:/SomethingOrNothing
			return false;
		}
		else
		{
			// InPath == RelativeComponent/SomethingOrNothing
			return true;
		}
	}
	else
	{
		// InPath == SomethingOrNothing
		return true;
	}
}

static void NormalizeAndCollapseDirectories(FStringBuilderBase& InOutPath)
{
	FPathViews::NormalizeFilename(InOutPath);
	FPathViews::CollapseRelativeDirectories(InOutPath);

	if (InOutPath.Len() == 0)
	{
		// Empty path is not absolute, and '/' is the best guess across all the platforms.
		// This substituion is not valid for Windows of course; however CollapseRelativeDirectories() will not produce an empty
		// absolute path on Windows as it takes care not to remove the drive letter.
		InOutPath << '/';
	}
}

void FPathViews::ToAbsolutePathInline(FStringView BasePath, FStringBuilderBase& InOutPath)
{
	check(BasePath.Len() > 0);

	if (IsRelativePath(InOutPath))
	{
		if (BasePath.EndsWith(TEXT('/')) || BasePath.EndsWith(TEXT('\\')))
		{
			InOutPath.Prepend(BasePath);
		}
		else
		{
			TStringBuilder<128> BaseWithSlash;
			BaseWithSlash << BasePath << '/';
			InOutPath.Prepend(BaseWithSlash);
		}
	}

	NormalizeAndCollapseDirectories(InOutPath);
}

void FPathViews::ToAbsolutePathInline(FStringBuilderBase& InOutPath)
{
	ToAbsolutePathInline(FStringView(FPlatformProcess::BaseDir()), InOutPath);
}

void FPathViews::ToAbsolutePath(FStringView BasePath, FStringView InPath, FStringBuilderBase& OutPath)
{
	if (int32 OriginalLen = OutPath.Len())
	{
		// Use temporary to avoid normalizing existing string builder contents
		TStringBuilder<256> AbsolutePath;
		ToAbsolutePath(BasePath, InPath, AbsolutePath);
		OutPath.Append(AbsolutePath);
	}
	else
	{
		if (IsRelativePath(InPath))
		{
			OutPath << BasePath;
		}

		FPathViews::Append(OutPath, InPath);

		NormalizeAndCollapseDirectories(OutPath);
	}
}

void FPathViews::ToAbsolutePath(FStringView InPath, FStringBuilderBase& OutPath)
{
	ToAbsolutePath(FStringView(FPlatformProcess::BaseDir()), InPath, OutPath);
}

void FPathViews::NormalizeFilename(FStringBuilderBase& InOutPath)
{
	Algo::Replace(MakeArrayView(InOutPath), TEXT('\\'), TEXT('/'));
	FPlatformMisc::NormalizePath(InOutPath);
}

void FPathViews::NormalizeDirectoryName(FStringBuilderBase& InOutPath)
{
	Algo::Replace(MakeArrayView(InOutPath), TEXT('\\'), TEXT('/'));
	InOutPath.RemoveSuffix(HasRedundantTerminatingSeparator(InOutPath.ToView()) ? 1 : 0);
	FPlatformMisc::NormalizePath(InOutPath);
}

static int32 FindNext(FStringView View, int32 FromIdx, FStringView Search)
{
	int32 RelativeIdx = UE::String::FindFirst(View.RightChop(FromIdx), Search);
	return RelativeIdx == INDEX_NONE ? INDEX_NONE : RelativeIdx + FromIdx;
}

static bool Contains(FStringView Str, TCHAR Char)
{
	int32 Dummy;
	return Str.FindChar(Char, Dummy);
}

static void RemoveAll(FStringBuilderBase& String, FStringView Substring)
{
	int32 Pos = String.Len();
	while (true)
	{
		Pos = UE::String::FindLast(String.ToView().Left(Pos), Substring);
		
		if (Pos == INDEX_NONE)
		{
			break;
		}

		String.RemoveAt(Pos, Substring.Len());
	}
}

namespace UE4PathViews_Private
{

void CollapseRemoveAt(FStringBuilderBase& Builder, int32 Index, int32 Count)
{
	Builder.RemoveAt(Index, Count);
}

void CollapseRemoveAt(FString& String, int32 Index, int32 Count)
{
	String.RemoveAt(Index, Count, EAllowShrinking::No);
}

template<typename WritableStringType>
void CollapseRelativeDirectoriesImplTemplate(WritableStringType& InOutPath,
	bool bCollapseAllPossible, bool& bOutAllCollapsed)
{
	constexpr FStringView DotDot(TEXTVIEW(".."));
	constexpr FStringView SlashDot(TEXTVIEW("/."));
	constexpr FStringView DotSlash(TEXTVIEW("./"));
	constexpr FStringView SlashDotDot(TEXTVIEW("/.."));

	bOutAllCollapsed = true;
	int32 StartOfCollapsibleDirs = 0;

	// If the path has a colon before the first slash, move past it
	FStringView Path = FStringView(InOutPath);
	int32 FirstSlashOrColonIndex = UE::String::FindFirstOfAnyChar(Path, { ':', '/' });
	if (FirstSlashOrColonIndex != INDEX_NONE && Path[FirstSlashOrColonIndex] == ':')
	{
		StartOfCollapsibleDirs = FirstSlashOrColonIndex + 1;
	}

	// We look for extraneous '.' paths by searching for "/." That will not handle a string with
	// a leading "./" or "D:./"; collapse that now.
	FStringView PathAfterColon = Path.RightChop(StartOfCollapsibleDirs);
	if (PathAfterColon.StartsWith(DotSlash, ESearchCase::CaseSensitive))
	{
		CollapseRemoveAt(InOutPath, StartOfCollapsibleDirs, DotSlash.Len());
	}
	// Searching for /. also incorrectly makes the string empty if the string is exactly "/.". Skip over it if so.
	else if (PathAfterColon == SlashDot)
	{
		StartOfCollapsibleDirs += SlashDot.Len();
	}

	int32 NextScanIndex = StartOfCollapsibleDirs;

	for (;;)
	{
		Path = FStringView(InOutPath);
		// When NextScanIndex == Path.Len we are finished
		if (NextScanIndex == Path.Len())
		{
			break;
		}

		// If remaining text after StartOfCollapsibleDirs starts with .. mark that we could not convert it
		// and move past it so we do not try to collapse it in the following .. 
		if (NextScanIndex == StartOfCollapsibleDirs)
		{
			FStringView CollapsibleText = Path.RightChop(StartOfCollapsibleDirs);
			if (CollapsibleText.StartsWith(DotDot, ESearchCase::CaseSensitive) &&
				// But don't mark that we could not convert it if it is a directory name that starts with ..,
				// such as "..whodoesthis/".
				(CollapsibleText.Len() == DotDot.Len() || CollapsibleText[DotDot.Len()] == '/'))
			{
				StartOfCollapsibleDirs += DotDot.Len();
				bOutAllCollapsed = false;
				if (!bCollapseAllPossible)
				{
					return;
				}
				continue;
			}
		}

		// If there are no "/."s left then we're done
		FStringView ScanText = Path.RightChop(NextScanIndex);
		int32 Index = UE::String::FindFirst(ScanText, SlashDot, ESearchCase::CaseSensitive);
		if (Index == INDEX_NONE)
		{
			break;
		}
		Index += NextScanIndex;

		// If we have found /./, or a trailing /., remove the spurious /.
		if (Index + SlashDot.Len() == Path.Len() || Path[Index + SlashDot.Len()] == '/')
		{
			CollapseRemoveAt(InOutPath, Index, SlashDot.Len());
			continue;
		}

		// If we have found a folder beginning with a dot (/.gitignore/) move past it
		if (Path[Index + SlashDot.Len()] != '.')
		{
			NextScanIndex = Index + SlashDot.Len() + 1;
			continue;
		}

		// If we have found a folder beginning with a pair of dots (/..whodoesthis/) move past it
		if (Index + SlashDotDot.Len() < Path.Len() && Path[Index + SlashDotDot.Len()] != '/')
		{
			NextScanIndex = Index + SlashDotDot.Len() + 1;
			continue;
		}

		// We have now found a substring starting at Index that is /../ or a trailing /..

		// If our discovered /../ is at StartOfCollapsibleDirs we cannot convert it; move past it
		if (Index == StartOfCollapsibleDirs)
		{
			StartOfCollapsibleDirs = Index + SlashDotDot.Len();
			NextScanIndex = StartOfCollapsibleDirs;
			bOutAllCollapsed = false;
			if (!bCollapseAllPossible)
			{
				return;
			}
			continue;
		}

		// If there was no previous slash, handle the case "d:foo/../child" by removing "foo/../"
		// If there was a previous slash, handle the case "d:/before/foo/../child" by removing "/foo/.."
		// Find the previous slash before the slash at Index.
		bool bHasLeadingSlash = true;
		int32 PreviousSeparatorIndex = -1;
		int32 RemovalStart = 0;
		FStringView PreviousText = Path.Left(Index);

		bool bFoundPreviousSeparator = PreviousText.FindLastChar('/', PreviousSeparatorIndex);

		if (bFoundPreviousSeparator)
		{
			// Handle a special case in unnormalized paths.
			// CollapseRelativeDirectories can be called on unnormalized paths with spurious duplicate separators,
			// such as "d:/root/Engine///////../..". We are not required to fixup all duplicate separators, but we
			// are expected to handle the collapse of .. as if the duplicate separators were removed. If
			// PreviousSeparatorIndex is a duplicate separator (and not the special case of allowed duplicate
			// separator at beginning of path), then ignore it and remove it and keep searching for the earlier
			// duplicate separator before it.
			if (PreviousSeparatorIndex == Index - 1)
			{
				// Move PreviousSeparatorIndex to the beginning of the slashes in the sequence foo////..
				while (PreviousSeparatorIndex > 0 && PreviousText[PreviousSeparatorIndex - 1] == '/')
				{
					--PreviousSeparatorIndex;
				} 

				// Note that we should be prevented from collpasing the valid occurrence of // at beginning of string
				// by StartOfCollapsibleDirs being set to point beyond that, but due to legacy behavior we 
				// allow the collapse of the initial / or //. When we stop allowing that, we need to set
				// bFoundPreviousSeparator to false here instead of true.
				if (PreviousSeparatorIndex == 0)
				{
					bFoundPreviousSeparator = true;
				}
				else
				{
					// Now mark for removal all of those slashes, AND the directory foo which comes before them.
					// To remove the directory foo, find the last / before the new PreviousSeparatorIndex
					PreviousText = Path.Left(PreviousSeparatorIndex);
					bFoundPreviousSeparator = PreviousText.FindLastChar('/', PreviousSeparatorIndex);
					// There is no need to recursively test for new PreviousSeparator == PreviousSeparatorIndex - 1,
					// because we know it does not. We moved to the start of the sequence of /// and we now have a
					// non-slash in PreviousSeparatorIndex[-1].
				}
			}
		}
		if (bFoundPreviousSeparator)
		{
			RemovalStart = PreviousSeparatorIndex;
		}
		else
		{
			// If there is no previous slash in the string, then remove all text from start of string, after the
			// initial colon. We use StartOfCollapsibleDirs as our first index after colon. We can rely on that,
			// even though other causes move StartOfCollapsibleDirs, because all of those other causes occur only
			// when there is a slash earlier in the string.
			RemovalStart = StartOfCollapsibleDirs;
			bHasLeadingSlash = false;
		}

		// The first index of InOutPath we are allowed to remove, given that we might have moved past some leading text,
		// is StartOfCollapsibleDirs. If RemovalStart < StartOfCollapsibleDirs then it includes text we cannot convert.
		if (RemovalStart < StartOfCollapsibleDirs)
		{
			StartOfCollapsibleDirs = Index + SlashDotDot.Len();
			NextScanIndex = StartOfCollapsibleDirs;
			bOutAllCollapsed = false;
			if (!bCollapseAllPossible)
			{
				return;
			}
			continue;
		}

		// We now have a trailing "/foo/.." or "/foo/../", or "^foo/.." or "^foo/../" (^ == start of text)
		// starting at RemovalStart, with the second slash at Index. Remove "/foo/.." or "foo/.." or "^foo/../"
		int32 RemoveLength = Index - RemovalStart + SlashDotDot.Len();
		if (!bHasLeadingSlash && RemovalStart + RemoveLength < Path.Len())
		{
			// If the removed text starts at beginning of path with no leading slash, also remove the slash after ..
			++RemoveLength;
		}
		CollapseRemoveAt(InOutPath, RemovalStart, RemoveLength);
		NextScanIndex = RemovalStart;
	}
}

void CollapseRelativeDirectoriesImpl(FStringBuilderBase& InOutPath, bool bCollapseAllPossible, bool& bOutAllCollapsed)
{
	CollapseRelativeDirectoriesImplTemplate(InOutPath, bCollapseAllPossible, bOutAllCollapsed);
}

void CollapseRelativeDirectoriesImpl(FString& InOutPath, bool bCollapseAllPossible, bool& bOutAllCollapsed)
{
	CollapseRelativeDirectoriesImplTemplate(InOutPath, bCollapseAllPossible, bOutAllCollapsed);
	InOutPath.TrimToNullTerminator();
}

}

bool FPathViews::CollapseRelativeDirectories(FStringBuilderBase& InOutPath, bool bCollapseAllPossible)
{
	bool bAllCollapsed;
	UE4PathViews_Private::CollapseRelativeDirectoriesImpl(InOutPath, bCollapseAllPossible, bAllCollapsed);
	return bAllCollapsed;
}

void FPathViews::RemoveDuplicateSlashes(FStringBuilderBase& InOutPath)
{
	int32 DoubleSlashIdx = UE::String::FindFirst(InOutPath.ToView(), TEXTVIEW("//"));
	if (DoubleSlashIdx == INDEX_NONE)
	{
		return;
	}

	TCHAR* WriteIt = InOutPath.GetData() + DoubleSlashIdx + 1;
	TCHAR* ReadIt = WriteIt + 1;
	for (TCHAR* End = InOutPath.GetData() + InOutPath.Len(), LastChar = TEXT('/'); ReadIt != End; LastChar = *ReadIt++)
	{
		if ((*ReadIt != TEXT('/')) | (LastChar != TEXT('/')))
		{
			*WriteIt++ = *ReadIt;
		}
	}

	*WriteIt = TEXT('\0');
	InOutPath.RemoveSuffix(int32(ReadIt - WriteIt));
}

void FPathViews::SplitFirstComponent(FStringView InPath, FStringView& OutFirstComponent, FStringView& OutRemainder)
{
	using namespace UE4PathViews_Private;

	if (const TCHAR* EndPos = Algo::FindByPredicate(InPath, IsSlashOrBackslash))
	{
		const int32 FirstLen = UE_PTRDIFF_TO_INT32(EndPos - InPath.GetData());
		if (FirstLen == 0)
		{
			// Path starts with /; it may be either /Foo or //Foo
			if (InPath.Len() == 1)
			{
				// InPath == /
				// FirstComponent = /
				// Remainder = <Empty>
				OutFirstComponent = InPath;
				OutRemainder.Reset();
			}
			else if (IsSlashOrBackslash(InPath[1]))
			{
				// InPath == //SomethingOrNothing
				// FirstComponent = //
				// Remainder = SomethingOrNothing
				OutFirstComponent = InPath.Left(2);
				OutRemainder = InPath.RightChop(2);
			}
			else
			{
				// InPath == /SomethingOrNothing
				// FirstComponent = /
				// Remainder = SomethingOrNothing
				OutFirstComponent = InPath.Left(1);
				OutRemainder = InPath.RightChop(1);
			}
		}
		else if (InPath[FirstLen - 1] == TEXT(':'))
		{
			// InPath == Volume:/SomethingOrNothing
			// FirstComponent = Volume:/
			// Remainder = SomethingOrNothing
			OutFirstComponent = InPath.Left(FirstLen + 1);
			OutRemainder = InPath.RightChop(FirstLen + 1);
		}
		else
		{
			// InPath == RelativeComponent/SomethingOrNothing
			// FirstComponent = RelativeComponent
			// Remainder = SomethingOrNothing
			OutFirstComponent = InPath.Left(FirstLen);
			OutRemainder = InPath.RightChop(FirstLen + 1);
		}
	}
	else
	{
		// InPath == SomethingOrNothing
		// FirstComponent = SomethingOrNothing
		// Remainder = <Empty>
		OutFirstComponent = InPath;
		OutRemainder.Reset();
	}
	// If there were invalid duplicate slashes after the first component, remove them since starting with a slash
	// is not a valid relpath
	while (OutRemainder.Len() > 0 && IsSlashOrBackslash(OutRemainder[0]))
	{
		OutRemainder.RightChopInline(1);
	}
}

bool FPathViews::IsDriveSpecifierWithoutRoot(FStringView InPath)
{
	const int32 PathLen = InPath.Len();
	const TCHAR* const PathData = InPath.GetData();
	for (int32 Index = 0; Index < PathLen; ++Index)
	{
		TCHAR C = PathData[Index];
		if (UE4PathViews_Private::IsSlashOrBackslash(C))
		{
			// '/' or '/root' or 'root/' or 'root/remainder'
			return false;
		}
		if (C == ':')
		{
			++Index;
			if (Index == PathLen)
			{
				// 'D:'
				return true;
			}
			C = PathData[Index];
			if (C == ':')
			{
				// 'D::root'
				// Path is even more invalid: two colons are not allowed. Arbitrarily return false for this case.
				return false;
			}
			if (UE4PathViews_Private::IsSlashOrBackslash(C))
			{
				// 'D:/root'
				return false;
			}
			// 'D:root'
			return true;
		}
	}
	// 'root'
	return false;
}

void FPathViews::SplitVolumeSpecifier(FStringView InPath, FStringView& OutVolumeSpecifier, FStringView& OutRemainder)
{
	const int32 PathLen = InPath.Len();
	const TCHAR* const PathData = InPath.GetData();

	if (PathLen == 0)
	{
		OutVolumeSpecifier.Reset();
		OutRemainder.Reset();
		return;
	}

	if (PathData[0] == ':')
	{
		OutVolumeSpecifier = InPath.Left(1);
		if (PathLen == 1)
		{
			// ':'
			OutRemainder.Reset();
		}
		else
		{
			// ':remainder'
			OutRemainder = InPath.RightChop(1);
		}
		return;
	}

	if (PathLen == 1)
	{
		// '/' or 'D'
		OutVolumeSpecifier.Reset();
		OutRemainder = InPath;
		return;
	}

	if (UE4PathViews_Private::IsSlashOrBackslash(PathData[0]))
	{
		if (UE4PathViews_Private::IsSlashOrBackslash(PathData[1]))
		{
			// '//' '///' or '//volume or //volume/remainder
			if (PathLen == 2)
			{
				OutVolumeSpecifier = InPath;
				OutRemainder.Reset();
				return;
			}

			// //////volume/remainder -> { '//////volume', 'remainder' }
			int32 SlashSlashEnd = 2;
			while (SlashSlashEnd < PathLen && UE4PathViews_Private::IsSlashOrBackslash(PathData[SlashSlashEnd]))
			{
				++SlashSlashEnd;
			}

			const TCHAR* VolumeEnd = Algo::FindByPredicate(FStringView(PathData + SlashSlashEnd, PathLen-SlashSlashEnd),
				UE4PathViews_Private::IsSlashOrBackslash);
			if (!VolumeEnd)
			{
				// '//' or '//volume'
				OutVolumeSpecifier = InPath;
				OutRemainder.Reset();
				return;
			}

			// '//' or '/////' or '//volume/remainder
			OutVolumeSpecifier = InPath.Left(static_cast<int32>(VolumeEnd - PathData));
			OutRemainder = InPath.RightChop(OutVolumeSpecifier.Len());
			return;
		}

		// '/remainder' or '/:'
		OutVolumeSpecifier.Reset();
		OutRemainder = InPath;
		return;
	}

	if (PathData[1] == ':')
	{
		// 'D:' or 'D:remainder'
		OutVolumeSpecifier = InPath.Left(2);
		OutRemainder = InPath.RightChop(2);
		return;
	}

	// root or root/remainder or drive:remainder
	for (int32 Index = 2; Index < PathLen; ++Index)
	{
		TCHAR C = PathData[Index];
		if (UE4PathViews_Private::IsSlashOrBackslash(C))
		{
			// 'root/remainder'
			OutVolumeSpecifier.Reset();
			OutRemainder = InPath;
			return;
		}
		if (C == ':')
		{
			++Index;
			OutVolumeSpecifier = InPath.Left(Index);
			if (Index == PathLen)
			{
				// 'drive:'
				OutRemainder.Reset();
			}
			else
			{
				// 'drive:remainder'
				OutRemainder = InPath.RightChop(Index);
			}
			return;
		}
	}

	// 'root'
	OutVolumeSpecifier.Reset();
	OutRemainder = InPath;
}

void FPathViews::AppendPath(FStringBuilderBase& InOutPath, FStringView AppendPath)
{
	using namespace UE4PathViews_Private;

	if (AppendPath.Len() == 0)
	{
	}
	else if (IsRelativePath(AppendPath))
	{
		if (InOutPath.Len() > 0 && !IsSlashOrBackslash(InOutPath.LastChar()))
		{
			InOutPath << TEXT('/');
		}
		InOutPath << AppendPath;
	}
	else
	{
		InOutPath.Reset();
		InOutPath << AppendPath;
	}
}

FStringView FPathViews::GetMountPointNameFromPath(const FStringView InPath, bool* bOutHadClassesPrefix /*= nullptr*/, bool bInWithoutSlashes /*= true*/)
{
	FStringView MountPointStringView;
	if (InPath.Len() > 0 && InPath[0] == '/')
	{
		int32 SecondForwardSlash = InPath.Len();
		int32 WithoutSlashes = bInWithoutSlashes ? 1 : 0;
		if (FStringView(InPath.GetData() + 1, InPath.Len() - 1).FindChar(TEXT('/'), SecondForwardSlash))
		{
			MountPointStringView = FStringView(InPath.GetData() + WithoutSlashes, SecondForwardSlash + 1 - WithoutSlashes);
		}
		else
		{
			MountPointStringView = FStringView(InPath.GetData() + WithoutSlashes, InPath.Len() - WithoutSlashes);
		}

		static const FString ClassesPrefix = "Classes_";
		const bool bHasClassesPrefix = MountPointStringView.StartsWith(ClassesPrefix);
		if (bHasClassesPrefix)
		{
			MountPointStringView.RightInline(MountPointStringView.Len() - ClassesPrefix.Len());
		}

		if (bOutHadClassesPrefix)
		{
			*bOutHadClassesPrefix = bHasClassesPrefix;
		}
	}
	return MountPointStringView;
}