// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StringFwd.h"
#include "CoreTypes.h"

class FString;

namespace UE4PathViews_Private
{

void CollapseRelativeDirectoriesImpl(FString& InPath, bool bCollapseAllPossible, bool& bOutAllCollapsed);
void CollapseRelativeDirectoriesImpl(FStringBuilderBase& InPath, bool bCollapseAllPossible, bool& bOutAllCollapsed);

}