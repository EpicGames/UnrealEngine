// Copyright Epic Games, Inc. All Rights Reserved.

// Core includes.
#include "Misc/Paths.h"

#include "Containers/StringView.h"
#include "Containers/Utf8String.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFile.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Logging/LogMacros.h"
#include "Misc/App.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/DataDrivenPlatformInfoRegistry.h"
#include "Misc/EngineVersion.h"
#include "Misc/Guid.h"
#include "Misc/LazySingleton.h"
#include "Misc/Parse.h"
#include "Misc/PathViewsPrivate.h"
#include "Misc/ScopeLock.h"
#include "Misc/PathViews.h"
#include "String/Find.h"
#include "String/ParseTokens.h"
#include "UObject/NameTypes.h"

#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"
#include <fileapi.h>
#include "Windows/HideWindowsPlatformTypes.h"
#endif

DEFINE_LOG_CATEGORY_STATIC(LogPaths, Log, All);

#if !defined(SUPPORTS_LOGS_IN_USERDIR)
	#define SUPPORTS_LOGS_IN_USERDIR 0
#endif

struct FPaths::FStaticData
{
	FCriticalSection GameProjectFilePathLock;
	FString          GameProjectFilePath;

	FString         UserDirArg;
	FString         GameSavedDir;
	FString         EngineSavedDir;
	FString         ShaderDir;
	FString         UserFolder;
	TArray<FString> EngineLocalizationPaths;
	TArray<FString> EditorLocalizationPaths;
	TArray<FString> CookedEditorLocalizationPaths;
	TArray<FString> PropertyNameLocalizationPaths;
	TArray<FString> ToolTipLocalizationPaths;
	TArray<FString> GameLocalizationPaths;
	TArray<FString> RestrictedFolderNames;
	TArray<FString> RestrictedSlashedFolderNames;
	FString         RelativePathToRoot;

	bool bUserDirArgInitialized                    = false;
	bool bGameSavedDirInitialized                  = false;
	bool bEngineSavedDirInitialized                = false;
	bool bShaderDirInitialized                     = false;
	bool bUserFolderInitialized                    = false;
	bool bEngineLocalizationPathsInitialized       = false;
	bool bEditorLocalizationPathsInitialized       = false;
	bool bCookedEditorLocalizationPathsInitialized = false;
	bool bPropertyNameLocalizationPathsInitialized = false;
	bool bToolTipLocalizationPathsInitialized      = false;
	bool bGameLocalizationPathsInitialized         = false;
	bool bRestrictedFolderNamesInitialized         = false;
	bool bRestrictedSlashedFolderNamesInitialized  = false;
	bool bRelativePathToRootInitialized            = false;
};

/*-----------------------------------------------------------------------------
	Path helpers for retrieving game dir, engine dir, etc.
-----------------------------------------------------------------------------*/

namespace UE4Paths_Private
{
	auto IsSlashOrBackslash    = [](auto C) { return C == '/' || C == '\\'; };
	auto IsNotSlashOrBackslash = [](auto C) { return C != '/' && C != '\\'; };

	FString GetSavedDirSuffix(const FString& BaseDir, const TCHAR* CommandLineArgument)
	{
		FString Result = BaseDir + TEXT("Saved");

		FString NonDefaultSavedDirSuffix;
		if (FParse::Value(FCommandLine::Get(), CommandLineArgument, NonDefaultSavedDirSuffix))
		{
			for (int32 CharIdx = 0; CharIdx < NonDefaultSavedDirSuffix.Len(); ++CharIdx)
			{
				if (!FCString::Strchr(VALID_SAVEDDIRSUFFIX_CHARACTERS, NonDefaultSavedDirSuffix[CharIdx]))
				{
					NonDefaultSavedDirSuffix.RemoveAt(CharIdx, EAllowShrinking::No);
					--CharIdx;
				}
			}
		}

		if (!NonDefaultSavedDirSuffix.IsEmpty())
		{
			Result += TEXT("_") + NonDefaultSavedDirSuffix;
		}

		Result += TEXT("/");

		return Result;
	}

	FString GameSavedDir()
	{
		return GetSavedDirSuffix(FPaths::ProjectUserDir(), TEXT("-saveddirsuffix="));
	}

	FString EngineSavedDir()
	{
		return GetSavedDirSuffix(FPaths::EngineUserDir(), TEXT("-enginesaveddirsuffix="));
	}

	template <typename BasePathAsStringOrStringView>
	FString ConvertRelativePathToFullInternal(BasePathAsStringOrStringView&& BasePath, FString&& InPath)
	{
		FString FullyPathed;
		if ( FPaths::IsRelative(InPath) )
		{
			FullyPathed  = MoveTempIfPossible(BasePath);
			FullyPathed /= MoveTemp(InPath);
		}
		else
		{
			FullyPathed = MoveTemp(InPath);
		}

		FPaths::NormalizeFilename(FullyPathed);
		FPaths::CollapseRelativeDirectories(FullyPathed);

		if (FullyPathed.Len() == 0)
		{
			// Empty path is not absolute, and '/' is the best guess across all the platforms.
			// This substituion is not valid for Windows of course; however CollapseRelativeDirectories() will not produce an empty
			// absolute path on Windows as it takes care not to remove the drive letter.
			FullyPathed = TEXT("/");
		}

		return FullyPathed;
	}
}

namespace UE::Paths
{
	bool bIsComputingStaged = false;
}

bool FPaths::CanGetProjectDir()
{
	return !UE::Paths::bIsComputingStaged;
}

bool FPaths::IsStaged()
{
	static bool bHasInitialized = false;
	static bool bIsStaged;

	if (!bHasInitialized)
	{
		UE::Paths::bIsComputingStaged = true;

		bIsStaged =
#if !IS_PROGRAM
			FPlatformProperties::RequiresCookedData() ||
#endif
			FileExists(Combine(EngineConfigDir(), FString::Printf(TEXT("StagedBuild_%s.ini"), FApp::GetProjectName())));

		UE::Paths::bIsComputingStaged = false;
		bHasInitialized = true;
	}

	return bIsStaged;
}

bool FPaths::ShouldSaveToUserDir()
{
	static bool bShouldSaveToUserDir =
		FApp::IsInstalled()
		|| FParse::Param(FCommandLine::Get(), TEXT("SaveToUserDir"))
		|| FPlatformProcess::ShouldSaveToUserDir()
		|| !CustomUserDirArgument().IsEmpty();
		
	return bShouldSaveToUserDir;
}

FString FPaths::LaunchDir()
{
	return FString(FPlatformMisc::LaunchDir());
}

FString FPaths::EngineDir()
{
	return FString(FPlatformMisc::EngineDir());
}

FString FPaths::EngineUserDir()
{
	if (ShouldSaveToUserDir() || FApp::IsEngineInstalled())
	{
		return FPaths::Combine(FPlatformProcess::UserSettingsDir(), *FApp::GetEpicProductIdentifier(), *FEngineVersion::Current().ToString(EVersionComponent::Minor)) + TEXT("/");
	}
	else
	{
		return FPaths::EngineDir();
	}
}

FString FPaths::EngineVersionAgnosticUserDir()
{
	if (ShouldSaveToUserDir() || FApp::IsEngineInstalled())
	{
		return FPaths::Combine(FPlatformProcess::UserSettingsDir(), *FApp::GetEpicProductIdentifier(), TEXT("Common")) + TEXT("/");
	}
	else
	{
		return FPaths::EngineDir();
	}
}

FString FPaths::EngineContentDir()
{
	return FPaths::EngineDir() + TEXT("Content/");
}

FString FPaths::EngineConfigDir()
{
	return FPaths::EngineDir() + TEXT("Config/");
}

FString FPaths::EngineEditorSettingsDir()
{
#if IS_MONOLITHIC
	// monolithic editors don't want/need to share settings with the non-monolithic editors
	return FPaths::GeneratedConfigDir();
#else
	return FPaths::GameAgnosticSavedDir() + TEXT("Config/");
#endif
}

FString FPaths::EngineIntermediateDir()
{
	return FPaths::EngineDir() + TEXT("Intermediate/");
}

FString FPaths::EngineSavedDir()
{
	FStaticData& StaticData = TLazySingleton<FStaticData>::Get();
	if (!StaticData.bEngineSavedDirInitialized)
	{
		StaticData.EngineSavedDir = UE4Paths_Private::EngineSavedDir();
		StaticData.bEngineSavedDirInitialized = true;
	}
	return StaticData.EngineSavedDir;
}

FString FPaths::EnginePluginsDir()
{
	return FPaths::EngineDir() + TEXT("Plugins/");
}

FString FPaths::EngineDefaultLayoutDir()
{
	return FPaths::EngineConfigDir() + TEXT("Layouts/");
}

FString FPaths::EngineProjectLayoutDir()
{
	return FPaths::ProjectConfigDir() + TEXT("Layouts/");
}

FString FPaths::EngineUserLayoutDir()
{
	return FPaths::EngineEditorSettingsDir() + TEXT("Layouts/");
}

FString FPaths::EnterpriseDir()
{
	return FPaths::RootDir() + TEXT("Enterprise/");
}

FString FPaths::EnterprisePluginsDir()
{
	return EnterpriseDir() + TEXT("Plugins/");
}

FString FPaths::EnterpriseFeaturePackDir()
{
	return FPaths::EnterpriseDir() + TEXT("FeaturePacks/");
}

FString FPaths::ConvertPath(const FString& Path, EPathConversion Method, const TCHAR* ExtraData, const TCHAR* OverrideProjectDir)
{
	// Basic idea here is: 
	//   We have a path coming in that we want to get the correct corresponding extension/restricted location 
	//   So, depending on parameters, and if it's a program path or not (programs are special locations), we split
	//   up the path into a Prefix and Suffix, and insert the correct folders into the middle that matches the Conversion method
	// Note that ExtraData is currently only for PlatformExtensions, and contains the Platform name

	// NOTE: We can't test that a Project_ type starts with ProjectDir, because of this case of a Program:
	//   Path0 = ProjectDir() + "Config";
	//   Path1 = ConvertPath(Path0, Project_NoRedist);
	//   Path2 = ConvertPath(Path1, Project_PlatformExtension, "Mac");
	// This is valid, and will generate the correct output, but note that Path1 does _not_ start with ProjectDir (which is ../../../Engine/Programs/Foo):
	//   Path0 = ../../../Engine/Programs/Foo/Config
	//   Path1 = ../../../Engine/Restricted/NoRedist/Programs/Foo/Config
	//   Path2 = ../../../Engine/Restricted/NoRedist/Platforms/Mac/Programs/Foo/Config
	// Aren't Programs fun??
		
	const bool bIsProgram = OverrideProjectDir == nullptr ? IS_PROGRAM : FCString::Strstr(OverrideProjectDir, TEXT("/Programs/")) != nullptr;
	const bool bIsProjectConversion = (int)Method >= (int)EPathConversion::Project_First;
	const bool bIsEngineConversion = !bIsProjectConversion;
	
	FString Prefix, Suffix;
	// programs need special love for project types, where we need to split on whatever is before the Programs, NOT ProjectDir
	if (bIsProgram && bIsProjectConversion)
	{
		int ProgramsLoc = Path.Replace(TEXT("\\"), TEXT("/")).Find(TEXT("/Programs/"));
		Prefix = Path.Mid(0, ProgramsLoc + 1);
		Suffix = Path.Mid(ProgramsLoc + 1);
	}
	// handle the override case, simply
	else if (OverrideProjectDir != nullptr && (int)Method >= (int)EPathConversion::Project_First)
	{
		// skip over engine or project directory
		Prefix = OverrideProjectDir;
		// grab what's left
		Suffix = Path.Mid(Prefix.Len());
	}
	else
	{
		// removing these temporarily until we handle an absolute path to Engine/Project - the GetUnrealBuildToolExecutableFilename() function is passing
		// in an absolute path to Engine which causes Config path conversion to fail here
//		checkfSlow(!(bIsProjectConversion && !Path.StartsWith(FPaths::ProjectDir().TrimChar('/'))), TEXT("Called ConvertPath with a Platform_ method, but Path [%s] didn't start with FPaths::ProjectDir() [%s]. This is not supported"), *Path, *FPaths::ProjectDir());
//		checkfSlow(!(bIsEngineConversion && !Path.StartsWith(FPaths::EngineDir().TrimChar('/'))), TEXT("Called ConvertPath with a Engine_ method, but Path [%s] didn't start with FPaths::EngineDir() [%s]. This is not supported"), *Path, *FPaths::EngineDir());

		// skip over engine or project directory
		Prefix = (int)Method < (int)EPathConversion::Project_First ? FPaths::EngineDir() : FPaths::ProjectDir();
		// grab what's left
		Suffix = Path.Mid(Prefix.Len());
	}
	
	// we check we can skip appending Suffix if it's empty, otherwise, we will end up with an extra trailing /
	// but we do need to make sure we end with a / if the incoming path did
	if (!Suffix.EndsWith(TEXT("/")) && Path.EndsWith(TEXT("/")))
	{
		Suffix += TEXT("/");
	}
	bool bAppendSuffix = Suffix.Len() > 0;
	
	switch (Method)
	{
		case EPathConversion::Engine_PlatformExtension:
		case EPathConversion::Project_PlatformExtension:
			return bAppendSuffix ? FPaths::Combine(Prefix, TEXT("Platforms"), ExtraData, Suffix) : FPaths::Combine(Prefix, TEXT("Platforms"), ExtraData);
			
		case EPathConversion::Engine_NotForLicensees:
		case EPathConversion::Project_NotForLicensees:
			return bAppendSuffix ? FPaths::Combine(Prefix, TEXT("Restricted/NotForLicensees"), Suffix) : FPaths::Combine(Prefix, TEXT("Restricted/NotForLicensees"));
			
		case EPathConversion::Engine_NoRedist:
		case EPathConversion::Project_NoRedist:
			return bAppendSuffix ? FPaths::Combine(Prefix, TEXT("Restricted/NoRedist"), Suffix) : FPaths::Combine(Prefix, TEXT("Restricted/NoRedist"));

		case EPathConversion::Engine_LimitedAccess:
		case EPathConversion::Project_LimitedAccess:
			return bAppendSuffix ? FPaths::Combine(Prefix, TEXT("Restricted/LimitedAccess"), Suffix) : FPaths::Combine(Prefix, TEXT("Restricted/LimitedAccess"));
	}
	
	return TEXT("");
}


static void AddIfDirectoryExists(TArray<FString>& ExtensionDirs, FString&& Dir)
{
	if (FPaths::DirectoryExists(Dir))
	{
		ExtensionDirs.Add(Dir);
	}
}

static void GetExtensionDirsInternal(TArray<FString>& ExtensionDirs, const FString& BaseDir, const FString& SubDir, FPaths::EGetExtensionDirsFlags Flags)
{
	if (EnumHasAnyFlags(Flags, FPaths::EGetExtensionDirsFlags::WithBase))
	{
		AddIfDirectoryExists(ExtensionDirs, FPaths::Combine(BaseDir, SubDir));
	}

	if (EnumHasAnyFlags(Flags, FPaths::EGetExtensionDirsFlags::WithPlatforms))
	{
		const bool bCheckValid = EnumHasAnyFlags(Flags, FPaths::EGetExtensionDirsFlags::CheckPlatformsValid);

		FString PlatformExtensionBaseDir = FPaths::Combine(BaseDir, TEXT("Platforms"));
		for (const FString& PlatformName : FDataDrivenPlatformInfoRegistry::GetPlatformDirectoryNames(bCheckValid))
		{
			AddIfDirectoryExists(ExtensionDirs, FPaths::Combine(PlatformExtensionBaseDir, PlatformName, SubDir));
		}
	}

	if (EnumHasAnyFlags(Flags, FPaths::EGetExtensionDirsFlags::WithRestricted))
	{
		FString RestrictedBaseDir = FPaths::Combine(BaseDir, TEXT("Restricted"));
		IFileManager::Get().IterateDirectory(*RestrictedBaseDir, [&ExtensionDirs, SubDir, Flags](const TCHAR* FilenameOrDirectory, bool bIsDirectory)  -> bool
		{
			if (bIsDirectory)
			{
				GetExtensionDirsInternal(ExtensionDirs, FilenameOrDirectory, SubDir, Flags);
			}
			return true;
		});
	}
}

TArray<FString> FPaths::GetExtensionDirs(const FString& BaseDir, const FString& SubDir, bool bCheckValid)
{
	EGetExtensionDirsFlags Flags = EGetExtensionDirsFlags::Default;
	if (!bCheckValid)
	{
		Flags &= ~EGetExtensionDirsFlags::CheckPlatformsValid;
	}

	TArray<FString> ExtensionDirs;
	GetExtensionDirsInternal(ExtensionDirs, BaseDir, SubDir, Flags);
	return ExtensionDirs;
}

TArray<FString> FPaths::GetExtensionDirs(const FString& BaseDir, const FString& SubDir, EGetExtensionDirsFlags Flags)
{
	TArray<FString> ExtensionDirs;
	GetExtensionDirsInternal(ExtensionDirs, BaseDir, SubDir, Flags);
	return ExtensionDirs;
}

FString FPaths::RootDir()
{
	return FString(FPlatformMisc::RootDir());
}

FString FPaths::ProjectDir()
{
	return FString(FPlatformMisc::ProjectDir());
}

FString FPaths::ProjectUserDir()
{
	const FString& UserDirArg = CustomUserDirArgument();
	
	if (!UserDirArg.IsEmpty())
	{
		return UserDirArg;
	}

	if (ShouldSaveToUserDir())
	{
		// if defined, this will override both saveddirsuffix and enginesaveddirsuffix
#ifdef UE_SAVED_DIR_OVERRIDE
		return FPaths::Combine(FPlatformProcess::UserSettingsDir(), TEXT(UE_STRINGIZE(UE_SAVED_DIR_OVERRIDE))) + TEXT("/");
#else
		return FPaths::Combine(FPlatformProcess::UserSettingsDir(), FApp::GetProjectName()) + TEXT("/");
#endif
	}
	else
	{
		return FPaths::ProjectDir();
	}
}

FString FPaths::ProjectContentDir()
{
	return FPaths::ProjectDir() + TEXT("Content/");
}

FString FPaths::ProjectConfigDir()
{
	return FPaths::ProjectDir() + TEXT("Config/");
}

const FString& FPaths::ProjectSavedDir()
{
	FStaticData& StaticData = TLazySingleton<FStaticData>::Get();
	if (!StaticData.bGameSavedDirInitialized)
	{
		StaticData.GameSavedDir = UE4Paths_Private::GameSavedDir();
		StaticData.bGameSavedDirInitialized = true;
	}
	return StaticData.GameSavedDir;
}

FString FPaths::ProjectIntermediateDir()
{
	return ProjectUserDir() + TEXT("Intermediate/");
}

FString FPaths::ShaderWorkingDir()
{
	const FString& ShaderDirArg = CustomShaderDirArgument();

	if (!ShaderDirArg.IsEmpty())
	{
		return ShaderDirArg;
	}

	return FPlatformProcess::ShaderWorkingDir();
}

FString FPaths::ProjectPluginsDir()
{
	return FPaths::ProjectDir() + TEXT("Plugins/");
}

FString FPaths::ProjectModsDir()
{
	return FPaths::ProjectDir() + TEXT("Mods/");
}

bool FPaths::HasProjectPersistentDownloadDir()
{
	return FPlatformMisc::HasProjectPersistentDownloadDir();
}

FString FPaths::ProjectPersistentDownloadDir()
{
	return FPlatformMisc::GamePersistentDownloadDir();
}

FString FPaths::SourceConfigDir()
{
	return FPaths::ProjectDir() + TEXT("Config/");
}

FString FPaths::GeneratedConfigDir()
{
	return FPlatformMisc::GeneratedConfigDir();
}

FString FPaths::SandboxesDir()
{
	return FPaths::ProjectDir() + TEXT("Saved/Sandboxes");
}

FString FPaths::ProfilingDir()
{
	return FPaths::ProjectSavedDir() + TEXT("Profiling/");
}

FString FPaths::ScreenShotDir()
{
	return FPaths::ProjectSavedDir() + TEXT("Screenshots/") + FPlatformProperties::PlatformName() + TEXT("/");
}

FString FPaths::BugItDir()
{
	return FPaths::ProjectSavedDir() + TEXT("BugIt/") + FPlatformProperties::PlatformName() + TEXT("/");
}

FString FPaths::VideoCaptureDir()
{
	return FPaths::ProjectSavedDir() + TEXT("VideoCaptures/");
}

FString FPaths::AudioCaptureDir()
{
	return FPaths::ProjectSavedDir() + TEXT("AudioCaptures/");
}

FString FPaths::ProjectLogDir()
{
#if defined(OVERRIDE_LOG_DIRECTORY_PLATFORM)
	const FString* OverrideDir = OVERRIDE_LOG_DIRECTORY_PLATFORM::GetOverrideLogDirectory();
	if (OverrideDir != nullptr)
	{
		return *OverrideDir;
	}
#elif PLATFORM_MAC || SUPPORTS_LOGS_IN_USERDIR
	if (CustomUserDirArgument().IsEmpty())
	{
		return FPlatformProcess::UserLogsDir();
	}
#elif PLATFORM_ANDROID && USE_ANDROID_FILE
	const FString* OverrideDir = IAndroidPlatformFile::GetOverrideLogDirectory();
	if (OverrideDir != nullptr)
	{
		return *OverrideDir;
	}
#endif

	return FPaths::ProjectSavedDir() + TEXT("Logs/");
}

FString FPaths::AutomationDir()
{
	return FPaths::ProjectSavedDir() + TEXT("Automation/");
}

FString FPaths::AutomationTransientDir()
{
	return FPaths::AutomationDir() + TEXT("Tmp/");
}

FString FPaths::AutomationReportsDir()
{
	return FPaths::AutomationDir() + TEXT("Reports/");
}

FString FPaths::AutomationLogDir()
{
	return FPaths::AutomationDir() + TEXT("Logs/");
}

FString FPaths::CloudDir()
{
	return FPlatformMisc::CloudDir();
}

FString FPaths::GameDevelopersDir()
{
	FString ContentDir = FPaths::ProjectContentDir();
	FStringView DevelopersFolder = DevelopersFolderName();
	return FString::Printf(TEXT("%.*s%.*s/"), ContentDir.Len(), *ContentDir,
		DevelopersFolder.Len(), DevelopersFolder.GetData());
}

FStringView FPaths::DevelopersFolderName()
{
	return TEXTVIEW("Developers");
}

FString FPaths::GameUserDeveloperFolderName()
{
	FStaticData& StaticData = TLazySingleton<FStaticData>::Get();

	if(!StaticData.bUserFolderInitialized)
	{
		// The user folder is the user name without any invalid characters
		const FString InvalidChars = INVALID_LONGPACKAGE_CHARACTERS;
		const FString& UserName = FPlatformProcess::UserName();

		StaticData.UserFolder = UserName;

		for (int32 CharIdx = 0; CharIdx < InvalidChars.Len(); ++CharIdx)
		{
			const FString Char = InvalidChars.Mid(CharIdx, 1);
			StaticData.UserFolder = StaticData.UserFolder.Replace(*Char, TEXT("_"), ESearchCase::CaseSensitive);
		}

		StaticData.bUserFolderInitialized = true;
	}

	return StaticData.UserFolder;
}

FString FPaths::GameUserDeveloperDir()
{
	return FPaths::GameDevelopersDir() + GameUserDeveloperFolderName() + TEXT("/");
}

FString FPaths::DiffDir()
{
	return FPaths::ProjectSavedDir() + TEXT("Diff/");
}

const TArray<FString>& FPaths::GetEngineLocalizationPaths()
{
	FStaticData& StaticData = TLazySingleton<FStaticData>::Get();

	if(!StaticData.bEngineLocalizationPathsInitialized)
	{
		if(GConfig && GConfig->IsReadyForUse())
		{
			GConfig->GetArray( TEXT("Internationalization"), TEXT("LocalizationPaths"), StaticData.EngineLocalizationPaths, GEngineIni );
#if UE_ENABLE_ICU
			UE_CLOGF(!StaticData.EngineLocalizationPaths.Num(), LogInit, Warning, "No paths for engine localization data were specifed in the engine configuration.");
#endif
			StaticData.bEngineLocalizationPathsInitialized = true;
		}
		else
		{
			StaticData.EngineLocalizationPaths.AddUnique(TEXT("../../../Engine/Content/Localization/Engine")); // Hardcoded convention.
		}
	}

	return StaticData.EngineLocalizationPaths;
}

const TArray<FString>& FPaths::GetEditorLocalizationPaths()
{
	FStaticData& StaticData = TLazySingleton<FStaticData>::Get();

	if(!StaticData.bEditorLocalizationPathsInitialized)
	{
		if(GConfig && GConfig->IsReadyForUse())
		{
			GConfig->GetArray( TEXT("Internationalization"), TEXT("LocalizationPaths"), StaticData.EditorLocalizationPaths, GEditorIni );
			if(!StaticData.EditorLocalizationPaths.Num())
			{
				UE_LOGF(LogInit, Warning, "No paths for editor localization data were specifed in the editor configuration.");
			}
			StaticData.bEditorLocalizationPathsInitialized = true;
		}
		else
		{
			StaticData.EditorLocalizationPaths.AddUnique(TEXT("../../../Engine/Content/Localization/Editor")); // Hardcoded convention.
		}
	}

	return StaticData.EditorLocalizationPaths;
}

const TArray<FString>& FPaths::GetPropertyNameLocalizationPaths()
{
	FStaticData& StaticData = TLazySingleton<FStaticData>::Get();

	if(!StaticData.bPropertyNameLocalizationPathsInitialized)
	{
		if(GConfig && GConfig->IsReadyForUse())
		{
			GConfig->GetArray( TEXT("Internationalization"), TEXT("PropertyNameLocalizationPaths"), StaticData.PropertyNameLocalizationPaths, GEditorIni );
			if(!StaticData.PropertyNameLocalizationPaths.Num())
			{
				UE_LOGF(LogInit, Warning, "No paths for property name localization data were specifed in the editor configuration.");
			}
			StaticData.bPropertyNameLocalizationPathsInitialized = true;
		}
		else
		{
			StaticData.PropertyNameLocalizationPaths.AddUnique(TEXT("../../../Engine/Content/Localization/PropertyNames")); // Hardcoded convention.
		}
	}

	return StaticData.PropertyNameLocalizationPaths;
}

const TArray<FString>& FPaths::GetToolTipLocalizationPaths()
{
	FStaticData& StaticData = TLazySingleton<FStaticData>::Get();

	if(!StaticData.bToolTipLocalizationPathsInitialized)
	{
		if(GConfig && GConfig->IsReadyForUse())
		{
			GConfig->GetArray( TEXT("Internationalization"), TEXT("ToolTipLocalizationPaths"), StaticData.ToolTipLocalizationPaths, GEditorIni );
			if(!StaticData.ToolTipLocalizationPaths.Num())
			{
				UE_LOGF(LogInit, Warning, "No paths for tooltips localization data were specifed in the editor configuration.");
			}
			StaticData.bToolTipLocalizationPathsInitialized = true;
		}
		else
		{
			StaticData.ToolTipLocalizationPaths.AddUnique(TEXT("../../../Engine/Content/Localization/ToolTips")); // Hardcoded convention.
		}
	}

	return StaticData.ToolTipLocalizationPaths;
}

const TArray<FString>& FPaths::GetGameLocalizationPaths()
{
	FStaticData& StaticData = TLazySingleton<FStaticData>::Get();

	if(!StaticData.bGameLocalizationPathsInitialized)
	{
		if(GConfig && GConfig->IsReadyForUse())
		{
			GConfig->GetArray( TEXT("Internationalization"), TEXT("LocalizationPaths"), StaticData.GameLocalizationPaths, GGameIni );
#if UE_ENABLE_ICU
			UE_CLOGF(!StaticData.GameLocalizationPaths.Num(), LogPaths, Warning, "No paths for game localization data were specifed in the game configuration.");
#endif
			StaticData.bGameLocalizationPathsInitialized = true;
		}
	}


	return StaticData.GameLocalizationPaths;
}

FString FPaths::GetPlatformLocalizationFolderName()
{
	// Note: If you change this, also update StageLocalizationDataForTarget (CopyBuildToStagingDirectory.Automation.cs), ProjectImportExportInfo.PlatformLocalizationFolderName (LocalizationProvider.cs)
	return TEXT("Platforms");
}

const TArray<FString>& FPaths::GetRestrictedFolderNames()
{
	FStaticData& StaticData = TLazySingleton<FStaticData>::Get();

	if (!StaticData.bRestrictedFolderNamesInitialized)
	{
		StaticData.RestrictedFolderNames.Add(TEXT("LimitedAccess"));
		StaticData.RestrictedFolderNames.Add(TEXT("NotForLicensees"));
		StaticData.RestrictedFolderNames.Add(TEXT("NoRedist"));
		StaticData.RestrictedFolderNames.Add(TEXT("CarefullyRedist"));
		StaticData.RestrictedFolderNames.Add(TEXT("EpicInternal"));

		// Add confidential platforms
		for (FName PlatformStr : FDataDrivenPlatformInfoRegistry::GetConfidentialPlatforms())
		{
			StaticData.RestrictedFolderNames.Add(PlatformStr.ToString());
		}

		StaticData.bRestrictedFolderNamesInitialized = true;
	}

	return StaticData.RestrictedFolderNames;
}

bool FPaths::IsRestrictedPath(const FString& InPath)
{
	FStaticData& StaticData = TLazySingleton<FStaticData>::Get();

	if (!StaticData.bRestrictedSlashedFolderNamesInitialized)
	{
		// Add leading and trailing slashes to restricted folder names.
		FString LeadingSlash("/");

		for (const FString& FolderStr : GetRestrictedFolderNames())
		{
			StaticData.RestrictedSlashedFolderNames.Add(LeadingSlash + FolderStr + TEXT('/'));
		}

		StaticData.bRestrictedSlashedFolderNamesInitialized = true;
	}

	// Normalize path
	FString NormalizedPath(InPath);
	NormalizeFilename(NormalizedPath);

	// Ensure trailing forward slash
	NormalizedPath /= FString();

	for (const FString& SubDir : StaticData.RestrictedSlashedFolderNames)
	{
		if (NormalizedPath.Contains(SubDir))
		{
			return true;
		}
	}

	return false;
}

FString FPaths::GameAgnosticSavedDir()
{
	return EngineSavedDir();
}

FString FPaths::EngineSourceDir()
{
	return FPaths::EngineDir() + TEXT("Source/");
}

FString FPaths::GameSourceDir()
{
	return FPaths::ProjectDir() + TEXT("Source/");
}

FString FPaths::FeaturePackDir()
{
	return FPaths::RootDir() + TEXT("FeaturePacks/");
}

bool FPaths::IsProjectFilePathSet()
{
	FStaticData& StaticData = TLazySingleton<FStaticData>::Get();
	FScopeLock Lock(&StaticData.GameProjectFilePathLock);
	return !StaticData.GameProjectFilePath.IsEmpty();
}

FString FPaths::GetProjectFilePath()
{
	FStaticData& StaticData = TLazySingleton<FStaticData>::Get();
	FScopeLock Lock(&StaticData.GameProjectFilePathLock);
	return StaticData.GameProjectFilePath;
}

void FPaths::SetProjectFilePath( const FString& NewGameProjectFilePath )
{
	FStaticData& StaticData = TLazySingleton<FStaticData>::Get();
	FScopeLock Lock(&StaticData.GameProjectFilePathLock);
	StaticData.GameProjectFilePath = NewGameProjectFilePath;
	FPaths::NormalizeFilename(StaticData.GameProjectFilePath);
}

FString FPaths::FindCorrectCase(const FString& Path)
{
#if PLATFORM_WINDOWS
	// GetFilenameOnDisk on Windows will resolve directory junctions and resolving those here has negative consequences
	// for workflows that use a junction at their root (eg: p4 gets confused about paths and operations fail).
	// There is a way to get a case-accurate path on Windows without resolving directory junctions, but it is slow.
	// We can use it here for this one-off situation without causing all uses of GetFilenameOnDisk to be slower.
	TStringBuilder<MAX_PATH> Builder;
	FPathViews::IterateComponents(
		Path,
		[&Builder](FStringView CurrentPathComponent)
		{
			if (Builder.Len() != 0)
			{
				Builder.AppendChar(TEXT('/'));
			}

			// Any volume name should be upper case
			const bool bIsVolumeSegment = CurrentPathComponent.EndsWith(TEXT(':'));
			if (bIsVolumeSegment)
			{
				Builder.Append(FString(CurrentPathComponent).ToUpper());
				return;
			}

			int32 LenBeforeCurrentComponent = Builder.Len();
			Builder.Append(CurrentPathComponent);

			// Skip over all segments that are either empty or contain relative transforms, they should remain as-is
			const bool bIsIgnoredSegment = CurrentPathComponent.IsEmpty() || CurrentPathComponent.Equals(TEXTVIEW(".")) || CurrentPathComponent.Equals(TEXTVIEW(".."));
			if (bIsIgnoredSegment)
			{
				return;
			}

			WIN32_FIND_DATAW Data;
			HANDLE Handle = FindFirstFileW(StringCast<WIDECHAR>(*Builder, Builder.Len() + 1).Get(), &Data);
			if (Handle != INVALID_HANDLE_VALUE)
			{
				Builder.RemoveSuffix(Builder.Len() - LenBeforeCurrentComponent);
				Builder.Append(Data.cFileName);
				FindClose(Handle);
			}
		}
	);
	return FString(Builder);
#else
	return IFileManager::Get().GetFilenameOnDisk(*Path);
#endif
}

FString FPaths::GetExtension(const FString& InPath, UE::Paths::EFlags Flags)
{
	using namespace UE::Paths;

	const FString CleanFilename = GetCleanFilename(InPath);
	int32 DotPos = !EnumHasAnyFlags(Flags, EFlags::AllowCompoundExtension)
		? CleanFilename.Find(TEXT("."), ESearchCase::CaseSensitive, ESearchDir::FromEnd)
		: CleanFilename.Find(TEXT("."), ESearchCase::CaseSensitive, ESearchDir::FromStart);
	if (DotPos != INDEX_NONE)
	{
		return CleanFilename.Mid(DotPos + (EnumHasAnyFlags(Flags, EFlags::IncludeDot) ? 0 : 1));
	}

	return TEXT("");
}

FString FPaths::GetExtension(const FString& InPath, bool bIncludeDot)
{
	using namespace UE::Paths;

	return GetExtension(InPath, bIncludeDot ? EFlags::IncludeDot : EFlags::None);
}

FString FPaths::GetCleanFilename(const FString& InPath)
{
	static_assert(INDEX_NONE == -1, "INDEX_NONE assumed to be -1");

	int32 EndPos   = InPath.FindLastCharByPredicate(UE4Paths_Private::IsNotSlashOrBackslash) + 1;
	int32 StartPos = InPath.FindLastCharByPredicate(UE4Paths_Private::IsSlashOrBackslash) + 1;

	FString Result = (StartPos <= EndPos) ? InPath.Mid(StartPos, EndPos - StartPos) : "";
	return Result;
}

FString FPaths::GetCleanFilenameUtf8(const FUtf8String& InPath)
{
	int32 EndPos = InPath.FindLastCharByPredicate(UE4Paths_Private::IsNotSlashOrBackslash) + 1;
	int32 StartPos = InPath.FindLastCharByPredicate(UE4Paths_Private::IsSlashOrBackslash) + 1;

	FString Result = (StartPos <= EndPos) ? FString(InPath.Mid(StartPos, EndPos - StartPos)) : "";
	return Result;
}

FString FPaths::GetCleanFilename(FString&& InPath)
{
	static_assert(INDEX_NONE == -1, "INDEX_NONE assumed to be -1");

	int32 EndPos   = InPath.FindLastCharByPredicate(UE4Paths_Private::IsNotSlashOrBackslash) + 1;
	int32 StartPos = InPath.FindLastCharByPredicate(UE4Paths_Private::IsSlashOrBackslash) + 1;

	if (StartPos <= EndPos)
	{
		InPath.RemoveAt(EndPos, InPath.Len() - EndPos, EAllowShrinking::No);
		InPath.RemoveAt(0, StartPos, EAllowShrinking::No);
	}
	else
	{
		InPath.Empty();
	}

	return MoveTemp(InPath);
}

template<typename T>
FString GetBaseFilenameImpl(T&& InPath, bool bRemovePath)
{
	FString Wk = bRemovePath ? FPaths::GetCleanFilename(Forward<T>(InPath)) : Forward<T>(InPath);

	// remove the extension
	const int32 ExtPos = Wk.Find(TEXT("."), ESearchCase::CaseSensitive, ESearchDir::FromEnd);

	if (ExtPos != INDEX_NONE)
	{
		// determine the position of the path/leaf separator
		int32 LeafPos = INDEX_NONE;
		if (!bRemovePath)
		{
			LeafPos = Wk.FindLastCharByPredicate(UE4Paths_Private::IsSlashOrBackslash);
		}

		if (LeafPos == INDEX_NONE || ExtPos > LeafPos)
		{
			Wk.LeftInline(ExtPos);
		}
	}

	return Wk;
}

FString FPaths::GetBaseFilename(const FString& InPath, bool bRemovePath)
{
	return GetBaseFilenameImpl(InPath, bRemovePath);
}

FString FPaths::GetBaseFilename(FString&& InPath, bool bRemovePath)
{
	return GetBaseFilenameImpl(MoveTemp(InPath), bRemovePath);
}

FString FPaths::GetPath(const FString& InPath)
{
	int32 Pos = InPath.FindLastCharByPredicate(UE4Paths_Private::IsSlashOrBackslash);

	FString Result;
	if (Pos != INDEX_NONE)
	{
		Result = InPath.Left(Pos);
	}

	return Result;
}

FString FPaths::GetPath(FString&& InPath)
{
	int32 Pos = InPath.FindLastCharByPredicate(UE4Paths_Private::IsSlashOrBackslash);

	FString Result;
	if (Pos != INDEX_NONE)
	{
		InPath.RemoveAt(Pos, InPath.Len() - Pos, EAllowShrinking::No);
		Result = MoveTemp(InPath);
	}

	return Result;
}

FString FPaths::GetPathLeaf(const FString& InPath)
{
	static_assert(INDEX_NONE == -1, "INDEX_NONE assumed to be -1");

	int32 EndPos   = InPath.FindLastCharByPredicate(UE4Paths_Private::IsNotSlashOrBackslash) + 1;
	int32 StartPos = InPath.FindLastCharByPredicate(UE4Paths_Private::IsSlashOrBackslash, EndPos) + 1;

	FString Result = InPath.Mid(StartPos, EndPos - StartPos);
	return Result;
}

FString FPaths::GetPathLeaf(FString&& InPath)
{
	static_assert(INDEX_NONE == -1, "INDEX_NONE assumed to be -1");

	int32 EndPos   = InPath.FindLastCharByPredicate(UE4Paths_Private::IsNotSlashOrBackslash) + 1;
	int32 StartPos = InPath.FindLastCharByPredicate(UE4Paths_Private::IsSlashOrBackslash, EndPos) + 1;

	InPath.RemoveAt(EndPos, InPath.Len() - EndPos, EAllowShrinking::No);
	InPath.RemoveAt(0, StartPos, EAllowShrinking::No);

	return MoveTemp(InPath);
}

FString FPaths::ChangeExtension(const FString& InPath, const FString& InNewExtension)
{
	int32 Pos = INDEX_NONE;
	if (InPath.FindLastChar(TEXT('.'), Pos))
	{
		const int32 PathEndPos = InPath.FindLastCharByPredicate(UE4Paths_Private::IsSlashOrBackslash);
		if (PathEndPos != INDEX_NONE && PathEndPos > Pos)
		{
			// The dot found was part of the path rather than the name
			Pos = INDEX_NONE;
		}
	}

	if (Pos != INDEX_NONE)
	{
		FString Result = InPath.Left(Pos);

		if (InNewExtension.Len() && InNewExtension[0] != '.')
		{
			Result += '.';
		}

		Result += InNewExtension;

		return Result;
	}

	return InPath;
}

FString FPaths::SetExtension(const FString& InPath, const FString& InNewExtension)
{
	int32 Pos = INDEX_NONE;
	if (InPath.FindLastChar(TEXT('.'), Pos))
	{
		const int32 PathEndPos = InPath.FindLastCharByPredicate(UE4Paths_Private::IsSlashOrBackslash);
		if (PathEndPos != INDEX_NONE && PathEndPos > Pos)
		{
			// The dot found was part of the path rather than the name
			Pos = INDEX_NONE;
		}
	}

	FString Result = Pos == INDEX_NONE ? InPath : InPath.Left(Pos);

	if (InNewExtension.Len() && InNewExtension[0] != '.')
	{
		Result += '.';
	}

	Result += InNewExtension;

	return Result;
}

bool FPaths::FileExists(const FString& InPath)
{
	return IFileManager::Get().FileExists(*InPath);
}

bool FPaths::DirectoryExists(const FString& InPath)
{
	return IFileManager::Get().DirectoryExists(*InPath);
}

bool FPaths::IsDrive(const FString& InPath)
{
	FString ConvertedPathString = InPath;

	ConvertedPathString = ConvertedPathString.Replace(TEXT("/"), TEXT("\\"), ESearchCase::CaseSensitive);
	const TCHAR* ConvertedPath= *ConvertedPathString;

	// Does Path refer to a drive letter or BNC path?
	if (ConvertedPath[0] == TCHAR(0))
	{
		return true;
	}
	else if (FChar::ToUpper(ConvertedPath[0])!=FChar::ToLower(ConvertedPath[0]) && ConvertedPath[1]==TEXT(':') && ConvertedPath[2]==0)
	{
		return true;
	}
	else if (FCString::Strcmp(ConvertedPath,TEXT("\\"))==0)
	{
		return true;
	}
	else if (FCString::Strcmp(ConvertedPath,TEXT("\\\\"))==0)
	{
		return true;
	}
	else if (ConvertedPath[0]==TEXT('\\') && ConvertedPath[1]==TEXT('\\') && !FCString::Strchr(ConvertedPath+2,TEXT('\\')))
	{
		return true;
	}
	else
	{
		// Need to handle cases such as X:\A\B\..\..\C\..
		// This assumes there is no actual filename in the path (ie, not c:\DIR\File.ext)!
		FString TempPath(ConvertedPath);
		// Make sure there is a '\' at the end of the path
		if (TempPath.Find(TEXT("\\"), ESearchCase::CaseSensitive, ESearchDir::FromEnd) != (TempPath.Len() - 1))
		{
			TempPath += TEXT("\\");
		}

		FString CheckPath;
		int32 ColonSlashIndex = TempPath.Find(TEXT(":\\"), ESearchCase::CaseSensitive);
		if (ColonSlashIndex != INDEX_NONE)
		{
			// Remove the 'X:\' from the start
			CheckPath = TempPath.Right(TempPath.Len() - ColonSlashIndex - 2);
		}
		else
		{
			// See if the first two characters are '\\' to handle \\Server\Foo\Bar cases
			if (TempPath.StartsWith(TEXT("\\\\"), ESearchCase::CaseSensitive) == true)
			{
				CheckPath = TempPath.Right(TempPath.Len() - 2);
				// Find the next slash
				int32 SlashIndex = CheckPath.Find(TEXT("\\"), ESearchCase::CaseSensitive);
				if (SlashIndex != INDEX_NONE)
				{
					CheckPath.RightInline(CheckPath.Len() - SlashIndex  - 1, EAllowShrinking::No);
				}
				else
				{
					CheckPath.Reset();
				}
			}
		}

		if (CheckPath.Len() > 0)
		{
			// Replace any remaining '\\' instances with '\'
			CheckPath.ReplaceInline(TEXT("\\\\"), TEXT("\\"), ESearchCase::CaseSensitive);

			int32 CheckCount = 0;
			int32 SlashIndex = CheckPath.Find(TEXT("\\"), ESearchCase::CaseSensitive);
			while (SlashIndex != INDEX_NONE)
			{
				FString FolderName = CheckPath.Left(SlashIndex);
				if (FolderName == TEXT(".."))
				{
					// It's a relative path, so subtract one from the count
					CheckCount--;
				}
				else
				{
					// It's a real folder, so add one to the count
					CheckCount++;
				}
				CheckPath.RightInline(CheckPath.Len() - SlashIndex  - 1, EAllowShrinking::No);
				SlashIndex = CheckPath.Find(TEXT("\\"), ESearchCase::CaseSensitive);
			}

			if (CheckCount <= 0)
			{
				// If there were the same number or greater relative to real folders, it's the root dir
				return true;
			}
		}
	}

	// It's not a drive...
	return false;
}

#if WITH_EDITOR
const TCHAR* FPaths::GameFeatureRootPrefix()
{
	return TEXT("root:/");
}
#endif

bool FPaths::IsRelative(const FString& InPath)
{
	// The previous implementation of this function seemed to handle normalized and unnormalized paths, so this one does too for legacy reasons.
	const uint32 PathLen = InPath.Len();
	const bool IsRooted = PathLen &&
		((InPath[0] == '/') ||												// Root of the current directory on Windows, root on UNIX-likes.  Also covers "\\", considering normalization replaces "\\" with "//".
		(PathLen >= 2 && (													// Check it's safe to access InPath[1]!
			((InPath[0] == '\\') && (InPath[1] == '\\'))					// Root of the current directory on Windows. Also covers "\\" for UNC or "network" paths.
			|| (InPath[1] == ':' && FChar::IsAlpha(InPath[0]))				// Starts with "<DriveLetter>:"
#if WITH_EDITOR
			|| (InPath.StartsWith(GameFeatureRootPrefix(), ESearchCase::IgnoreCase))
#endif // WITH_EDITOR
			))
		);
	return !IsRooted;
}

void FPaths::PlatformIndependentNormalizePath(FString& Path, bool bRemoveDuplicateSlashes,
	int32& OutFirstColonIndex, int32& OutFirstSlashIndex)
{
	TCHAR* Start = Path.GetCharArray().GetData();
	TCHAR* ReadHead = Start;
	TCHAR* WriteHead = Start;
	TCHAR* End = ReadHead + Path.Len();
	OutFirstColonIndex = INDEX_NONE;
	OutFirstSlashIndex = INDEX_NONE;

	while (ReadHead < End)
	{
		TCHAR C = *ReadHead++;
		bool bWrite = true;

		// 1. Convert '\' to '/'.
		if (C == '\\')
		{
			C = '/';
		}

		// 2. Find FirstColonIndex and FirstSlashIndex for categorization
		if (C == '/' && OutFirstSlashIndex == INDEX_NONE)
		{
			OutFirstSlashIndex = static_cast<int32>(WriteHead - Start);
		}
		if (C == ':' && OutFirstColonIndex == INDEX_NONE)
		{
			OutFirstColonIndex = static_cast<int32>(WriteHead - Start);
		}

		// 3. RemoveDuplicateSlashes (note that this behavior should be identical to FPaths::RemoveDuplicateSlashes)
		if (bRemoveDuplicateSlashes && C == '/' && WriteHead > Start && WriteHead[-1] == '/')
		{
			bWrite = false;
		}

		if (bWrite)
		{
			*WriteHead++ = C;
		}
	}

	if (WriteHead < End)
	{
		*WriteHead++ = TEXT('\0');
		Path.TrimToNullTerminator();
	}
}

void FPaths::NormalizeFilename(FString& InPath)
{
	int32 FirstColonIndex;
	int32 FirstSlashIndex;
	PlatformIndependentNormalizePath(InPath, false /* bRemoveDuplicateSlashes */, FirstColonIndex, FirstSlashIndex);

	FPlatformMisc::NormalizePath(InPath);
}

void FPaths::NormalizeDirectoryName(FString& InPath)
{
	int32 FirstColonIndex;
	int32 FirstSlashIndex;
	PlatformIndependentNormalizePath(InPath, false /* bRemoveDuplicateSlashes */, FirstColonIndex, FirstSlashIndex);

	if (InPath.EndsWith(TEXT("/"), ESearchCase::CaseSensitive) 
		&& !InPath.EndsWith(TEXT("//"), ESearchCase::CaseSensitive)
		&& !InPath.EndsWith(TEXT(":/"), ESearchCase::CaseSensitive))
	{
		// overwrite trailing slash with terminator
		InPath.GetCharArray()[InPath.Len() - 1] = TEXT('\0');
		// shrink down
		InPath.TrimToNullTerminator();
	}

	FPlatformMisc::NormalizePath(InPath);
}

bool FPaths::CollapseRelativeDirectories(FString& InPath, bool bCollapseAllPossible)
{
	bool bAllCollapsed;
	UE4PathViews_Private::CollapseRelativeDirectoriesImpl(InPath, bCollapseAllPossible, bAllCollapsed);
	return bAllCollapsed;
}

FString FPaths::RemoveDuplicateSlashes(const FString& InPath)
{
	FString Result = InPath;
	RemoveDuplicateSlashes(Result);
	return MoveTemp(Result);
}

void FPaths::RemoveDuplicateSlashes(FString& InPath)
{
	TCHAR* Text = InPath.GetCharArray().GetData();
	if (!Text)
	{
		return;
	}
	const TCHAR* const TwoSlashStr = TEXT("//");
	const TCHAR SlashChr = TEXT('/');

	TCHAR* const EditStart = TCString<TCHAR>::Strstr(Text, TwoSlashStr);
	if (!EditStart)
	{
		return;
	}
	const TCHAR* const TextEnd = Text + InPath.Len();

	// Since we know we've found TwoSlashes, point the initial Write head at the spot where the second slash is (which we shall skip), and point the Read head at the next character after the second slash
	TCHAR* Write = EditStart + 1;	// The position to write the next character of the output
	const TCHAR* Read = Write + 1;	// The next character of the input to read

	for (; Read < TextEnd; ++Read)
	{
		TCHAR NextChar = *Read;
		if (Write[-1] != SlashChr || NextChar != SlashChr)
		{
			*Write++ = NextChar;
		}
		else
		{
			// Skip the NextChar when adding on a slash after an existing slash, e.g // or more generally before/////after
			//                                                                       WR                         W  R
		}
	}
	*Write = TEXT('\0');
	InPath.TrimToNullTerminator();
}

FString FPaths::CreateStandardFilename(const FString& InPath)
{
	// if this is an empty path, use the relative base dir
	if (InPath.Len() == 0)
	{
		FString BaseDir = FPlatformProcess::BaseDir();
		// if the base directory is nothing then this function will recurse infinitely instead of returning nothing.
		if (BaseDir.Len() == 0)
			return BaseDir;
		return FPaths::CreateStandardFilename(BaseDir);
	}

	// If we have a windows network path, leave it alone; we do not have a contract for what standardized
	// form of a network path looks like - do we keep all slashes as windows backslashes? Keep the first two
	// and convert the rest? Convert all of them including the first two?
	if (InPath.StartsWith(TEXTVIEW("\\\\")))
	{
		return InPath;
	}

	// Normalize and categorize the path characters before collapsing .. directories and putting it in standard form.
	FString NormalizedPath = InPath;
	int32 ColonIndex = -1;
	int32 SlashIndex = -1;
	PlatformIndependentNormalizePath(NormalizedPath, true /* bRemoveDuplicateSlashes */, ColonIndex, SlashIndex);
	bool bHasVolumeSpecifier = ColonIndex > 0 && (SlashIndex < 0 || ColonIndex < SlashIndex);

	const TCHAR* RootDirectory = FPlatformMisc::RootDir();
	const FStringView RootDirectoryView(RootDirectory);

	// look for paths that cannot be made relative, and are therefore normalized but not standardized.
	// 1. UNC (windows) network path would be one of these, but we have an even stricter rule and early exited above.
	// 2. windows volume path (e.g. 'd:\root2' or 'Storage:\root2') that doesn't start with base dir
	bool bCannotBeStandardized = bHasVolumeSpecifier && !NormalizedPath.StartsWith(RootDirectoryView);
	// 3. Unix style absolute path (e.g. `/home`) that doesn't start with base dir
	bCannotBeStandardized |= SlashIndex == 0 && !NormalizedPath.StartsWith(RootDirectoryView);

	// if it can't be standardized, normalize it but don't try to convert it to a relative path from root.
	if (bCannotBeStandardized)
	{
		FPlatformMisc::NormalizePath(NormalizedPath);
		CollapseRelativeDirectories(NormalizedPath, true /* bCollapseAllPossible */);
		return NormalizedPath;
	}

	// If the path starts with RelativePathToRoot, it is mostly done and just needs to be normalized.
	const FString& RelativePathToRoot = FPaths::GetRelativePathToRoot();
	if (NormalizedPath.StartsWith(RelativePathToRoot))
	{
		FPlatformMisc::NormalizePath(NormalizedPath);
		CollapseRelativeDirectories(NormalizedPath, true /* bCollapseAllPossible */);

		// Paths outside of the root path define their standardized path == absolute path.
		// A path might "start with" RelativePathToRoot, but still be outside the root path because
		// it adds on another "..". Check for that here before returning NormalizedPath based on the
		// assumption that it is a child path of root and already normalized.
		FStringView Suffix = FStringView(NormalizedPath).RightChop(RelativePathToRoot.Len());
		if (Suffix.Len() > 0 && Suffix[0] == '/')
		{
			Suffix = Suffix.RightChop(1);
		}
		if (!Suffix.StartsWith(TEXT("..")))
		{
			return NormalizedPath;
		}
	}

	// Make it an absolute path and then make that absolute path a subdirectory of the RelativePathToRoot.
	// This standardizes both incoming absolute paths, and relative paths that are relative to a subdirectory of
	// root rather than starting with the full relative to root path.
	// d:/root/Engine/Content                            -> ../../../Engine/Content
	// ../../Content                                     -> ../../../Engine/Content
	// d:/root/Engine/Binaries/Platform/UnrealEditor.exe -> ../../../Engine/Binaries/Platform/UnrealEditor.exe
	// UnrealEditor.exe                                  -> ../../../Engine/Binaries/Platform/UnrealEditor.exe
	FString FullPath = FPaths::ConvertRelativePathToFull(NormalizedPath);
	if (!FullPath.StartsWith(RootDirectoryView))
	{
		// Relative paths from the root directory that use .. to escape the root directory can end up here.
		// D:\root\..\AssetRegistryCache -> D:/AssetRegistryCache
		// Return their absolute path as the standard path.
		if (FullPath.Contains(RootDirectoryView))
		{
			// Unexpected that the absolute path of the RootDirectory appears at offset > 0 in the absolute path for
			// InPath. In legacy behavior we used to replace it with RelativePathToRoot, but that would likely be
			// invalid. Log a warning so that we can find this case where we deviate from legacy behavior if it exists.
			UE_LOGF(LogPaths, Warning,
				"CreateStandardFilename: Unexpected: InPath '%ls' transformed to '%ls' which contains but does not start with '%.*ls'.",
				*InPath, *FullPath, RootDirectoryView.Len(), RootDirectoryView.GetData());
		}
		return FullPath;
	}

	NormalizedPath.Reset();
	NormalizedPath += RelativePathToRoot;
	NormalizedPath += FStringView(FullPath).RightChop(RootDirectoryView.Len());
	return NormalizedPath;
}

void FPaths::MakeStandardFilename(FString& InPath)
{
	InPath = FPaths::CreateStandardFilename(InPath);
}

void FPaths::MakePlatformFilename( FString& InPath )
{
	InPath.ReplaceInline(TEXT( "\\" ), FPlatformMisc::GetDefaultPathSeparator(), ESearchCase::CaseSensitive);
	InPath.ReplaceInline(TEXT( "/" ), FPlatformMisc::GetDefaultPathSeparator(), ESearchCase::CaseSensitive);
}

bool FPaths::MakePathRelativeTo( FString& InPath, const TCHAR* InRelativeTo )
{
	FString Target = FPaths::ConvertRelativePathToFull(InPath);
	FString Source = FPaths::GetPath(FPaths::ConvertRelativePathToFull(InRelativeTo));

	Source.ReplaceInline(TEXT("\\"), TEXT("/"), ESearchCase::CaseSensitive);
	Target.ReplaceInline(TEXT("\\"), TEXT("/"), ESearchCase::CaseSensitive);

	const UE::String::EParseTokensOptions ParseOptions = UE::String::EParseTokensOptions::IgnoreCase |
		UE::String::EParseTokensOptions::SkipEmpty;
	TArray<FStringView, TInlineAllocator<16>> TargetArrayBuffer;
	UE::String::ParseTokens(Target, TEXTVIEW("/"), TargetArrayBuffer, ParseOptions);
	TArrayView<FStringView> TargetArray(TargetArrayBuffer);
	TArray<FStringView, TInlineAllocator<16>> SourceArrayBuffer;
	UE::String::ParseTokens(Source, TEXTVIEW("/"), SourceArrayBuffer, ParseOptions);
	TArrayView<FStringView> SourceArray(SourceArrayBuffer);

	if (TargetArray.Num() && SourceArray.Num())
	{
		// Check for being on different drives
		if ((TargetArray[0].Len() > 1) && (TargetArray[0][1] == TEXT(':')) && (SourceArray[0].Len() > 1) && (SourceArray[0][1] == TEXT(':')))
		{
			if (FChar::ToUpper(TargetArray[0][0]) != FChar::ToUpper(SourceArray[0][0]))
			{
				// The Target and Source are on different drives... No relative path available.
				return false;
			}
		}
	}

	while (TargetArray.Num() && SourceArray.Num() && TargetArray[0] == SourceArray[0])
	{
		TargetArray.RightChopInline(1);
		SourceArray.RightChopInline(1);
	}

	FStringView ParentDirSeparator = TEXTVIEW("../");
	FStringView DirSeparator = TEXTVIEW("/");
	int32 ResultsLen = SourceArray.Num() * ParentDirSeparator.Len() + (FMath::Max(TargetArray.Num(),1) - 1)*DirSeparator.Len();
	for (const FStringView& TargetDir : TargetArray)
	{
		ResultsLen += TargetDir.Len();
	}

	FString Result;
	Result.Reserve(ResultsLen);
	for (int32 Index = 0; Index < SourceArray.Num(); Index++)
	{
		Result += ParentDirSeparator;
	}
	for (int32 Index = 0; Index < TargetArray.Num(); Index++)
	{
		Result += TargetArray[Index];
		if (Index + 1 < TargetArray.Num())
		{
			Result += DirSeparator;
		}
	}
	
	InPath = MoveTemp(Result);
	return true;
}

FString FPaths::ConvertRelativePathToFull(const FString& InPath)
{
	return UE4Paths_Private::ConvertRelativePathToFullInternal(FStringView(FPlatformProcess::BaseDir()), FString(InPath));
}

FString FPaths::ConvertRelativePathToFull(FString&& InPath)
{
	return UE4Paths_Private::ConvertRelativePathToFullInternal(FStringView(FPlatformProcess::BaseDir()), MoveTemp(InPath));
}

FString FPaths::ConvertRelativePathToFull(const FString& BasePath, const FString& InPath)
{
	return UE4Paths_Private::ConvertRelativePathToFullInternal(BasePath, CopyTemp(InPath));
}

FString FPaths::ConvertRelativePathToFull(const FString& BasePath, FString&& InPath)
{
	return UE4Paths_Private::ConvertRelativePathToFullInternal(BasePath, MoveTemp(InPath));
}

FString FPaths::ConvertRelativePathToFull(FString&& BasePath, const FString& InPath)
{
	return UE4Paths_Private::ConvertRelativePathToFullInternal(MoveTemp(BasePath), CopyTemp(InPath));
}

FString FPaths::ConvertRelativePathToFull(FString&& BasePath, FString&& InPath)
{
	return UE4Paths_Private::ConvertRelativePathToFullInternal(MoveTemp(BasePath), MoveTemp(InPath));
}

FString FPaths::ConvertToSandboxPath( const FString& InPath, const TCHAR* InSandboxName )
{
	FString SandboxDirectory = FPaths::SandboxesDir() / InSandboxName;
	FPaths::NormalizeFilename(SandboxDirectory);
	
	FString RootDirectory = FPaths::RootDir();
	FPaths::CollapseRelativeDirectories(RootDirectory);
	FPaths::NormalizeFilename(RootDirectory);

	FString SandboxPath = FPaths::ConvertRelativePathToFull(InPath);
	if (!SandboxPath.StartsWith(RootDirectory))
	{
		UE_LOGF(LogInit, Fatal, "%ls does not start with %ls so this is not a valid sandbox path.", *SandboxPath, *RootDirectory);
	}
	check(SandboxPath.StartsWith(RootDirectory));
	SandboxPath.ReplaceInline(*RootDirectory, *SandboxDirectory);

	return SandboxPath;
}

FString FPaths::ConvertFromSandboxPath( const FString& InPath, const TCHAR* InSandboxName )
{
	FString SandboxDirectory =  FPaths::SandboxesDir() / InSandboxName;
	FPaths::NormalizeFilename(SandboxDirectory);
	FString RootDirectory = FPaths::RootDir();
	
	FString SandboxPath(InPath);
	check(SandboxPath.StartsWith(SandboxDirectory));
	SandboxPath.ReplaceInline(*SandboxDirectory, *RootDirectory);
	return SandboxPath;
}

FString FPaths::CreateTempFilename( const TCHAR* Path, const TCHAR* Prefix, const TCHAR* Extension )
{
	FString UniqueFilename;
	do
	{
		UniqueFilename = FPaths::Combine(Path, *FString::Printf(TEXT("%s%s%s"), Prefix, *FGuid::NewGuid().ToString(), Extension));
	}
	while (IFileManager::Get().FileSize(*UniqueFilename) >= 0);
	
	return UniqueFilename;
}

FString FPaths::GetInvalidFileSystemChars()
{
	// Windows has the most restricted file system, and since we're cross platform, we have to respect the limitations of the lowest common denominator
	// # isn't legal. Used for revision specifiers in P4/SVN, and also not allowed on Windows anyway
	// @ isn't legal. Used for revision/label specifiers in P4/SVN
	// ^ isn't legal. While the file-system won't complain about this character, Visual Studio will			
	static const TCHAR* RestrictedChars = TEXT("/?:&\\*\"<>|%#@^");
	return RestrictedChars;
}

FString FPaths::MakeValidFileName(const FString& InString, const TCHAR InReplacementChar /*= 0*/)
{
	const FString RestrictedChars = GetInvalidFileSystemChars();

	const int InLen = InString.Len();

	TArray<TCHAR> Output;
	Output.AddUninitialized(InLen + 1);

	// first remove all invalid chars
	for (int i = 0; i < InLen; i++)
	{
		int32 Unused = 0;
		if (RestrictedChars.FindChar(InString[i], Unused))
		{
			Output[i] = InReplacementChar;
		}
		else
		{
			Output[i] = InString[i];
		}
	}

	Output[InLen] = TEXT('\0');

	if (InReplacementChar == 0)
	{
		int CurrentChar = 0;

		// compact the string by replacing any null entries with the next non-null entry
		int iFill = 0;
		for (int iChar = 0; iChar < InLen; iChar++)
		{
			if (Output[iChar] == 0)
			{
				// adjust our fill index if we passed it
				if (iFill < iChar)
				{
					iFill = iChar;
				}

				// scan forward
				while (++iFill < InLen)
				{
					if (Output[iFill] != 0)
					{
						break;
					}
				}

				if (iFill < InLen)
				{
					// take this char and null it out
					Output[iChar] = Output[iFill];
					Output[iFill] = TEXT('\0');
				}
			}
		}
	}
	
	return FString(Output.GetData());
}

bool FPaths::ValidatePath(const FString& InPath, FText* OutReason)
{
	const FString RestrictedChars = GetInvalidFileSystemChars();
	static const TCHAR* RestrictedNames[] = {	TEXT("CON"), TEXT("PRN"), TEXT("AUX"), TEXT("CLOCK$"), TEXT("NUL"),
												TEXT("COM1"), TEXT("COM2"), TEXT("COM3"), TEXT("COM4"),   TEXT("COM5"), TEXT("COM6"), TEXT("COM7"), TEXT("COM8"), TEXT("COM9"),
												TEXT("LPT1"), TEXT("LPT2"), TEXT("LPT3"), TEXT("LPT4"),   TEXT("LPT5"), TEXT("LPT6"), TEXT("LPT7"), TEXT("LPT8"), TEXT("LPT9") };

	FString Standardized = InPath;
	NormalizeFilename(Standardized);
	CollapseRelativeDirectories(Standardized);
	// Remove duplicate slashes, to normalize the path,
	// but don't remove them at beginning so we can recognize network paths
	if (Standardized.StartsWith(TEXT("//")))
	{
		Standardized.RightChopInline(1, EAllowShrinking::No);
		RemoveDuplicateSlashes(Standardized);
		Standardized = TEXT("/") + MoveTemp(Standardized);
	}
	else
	{
		RemoveDuplicateSlashes(Standardized);
	}

	// Walk each part of the path looking for name errors
	// Allow colons in the first component
	bool bResult = true;
	bool bAllowNextTokenToBeDrive = true;
	FPathViews::IterateComponents(Standardized,
		[&bAllowNextTokenToBeDrive,&Standardized,&RestrictedChars,&bResult,&OutReason]
		(FStringView PathPart)
	{
		if (!bResult)
		{
			// Keep the first error and stop validating
			return;
		}
		bool bAllowDrive = bAllowNextTokenToBeDrive;
		bAllowNextTokenToBeDrive = false;
		if (PathPart.IsEmpty())
		{
			// We can get an empty path part due to duplicate slashes at the beginning or a
			// a terminating slash
			return;
		}

		// Check for invalid characters
		FString MatchedInvalidChars;
		for (TCHAR InvalidCharacter : RestrictedChars)
		{
			int32 UnusedIndex;
			if (PathPart.FindChar(InvalidCharacter, UnusedIndex))
			{
				if (InvalidCharacter == ':' && bAllowDrive && IsDrive(FString(PathPart)))
				{
					// Colons are allowed in drive specifiers
					continue;
				}
				if (InvalidCharacter == '?')
				{
					int64 StartPos = PathPart.GetData() - *Standardized;
					if (StartPos < 0 || StartPos > int64(Standardized.Len()))
					{
						StartPos = INDEX_NONE;
					}
					// ? is allowed at the beginning of the path to support windows-style long paths: "\\?\K:\LongPath"
					if (StartPos == 2 && PathPart.Len() == 1 && Standardized.StartsWith(TEXT("//"))
						&& Standardized.Len() > 4 && Standardized[3] == '/')
					{
						// Also allow a drive specifier in the next pathpart
						bAllowNextTokenToBeDrive = true;
						continue;
					}
				}

				MatchedInvalidChars += InvalidCharacter;
			}
		}

		if (MatchedInvalidChars.Len())
		{
			if (OutReason)
			{
				FFormatNamedArguments Args;
				Args.Add(TEXT("IllegalPathCharacters"), FText::FromString(MatchedInvalidChars));
				*OutReason = FText::Format(NSLOCTEXT("Core", "PathContainsInvalidCharacters", "Path may not contain the following characters: {IllegalPathCharacters}"), Args);
			}
			bResult = false;
			return;
		}

		// Check for reserved names
		for (const TCHAR* RestrictedName : RestrictedNames)
		{
			if (PathPart.Equals(RestrictedName, ESearchCase::IgnoreCase))
			{
				if (OutReason)
				{
					FFormatNamedArguments Args;
					Args.Add(TEXT("RestrictedName"), FText::FromString(RestrictedName));
					*OutReason = FText::Format(NSLOCTEXT("Core", "PathContainsRestrictedName", "Path may not contain a restricted name: {RestrictedName}"), Args);
				}
				bResult = false;
				return;
			}
		}
	});

	return bResult;
}

void FPaths::Split( const FString& InPath, FString& PathPart, FString& FilenamePart, FString& ExtensionPart )
{
	FStringView PathView;
	FStringView FilenameView;
	FStringView ExtensionView;

	FPathViews::Split(InPath, PathView, FilenameView, ExtensionView);
	PathPart = PathView;
	FilenamePart = FilenameView;
	ExtensionPart = ExtensionView;
}

const FString& FPaths::GetRelativePathToRoot()
{
	FStaticData& StaticData = TLazySingleton<FStaticData>::Get();

	if (!StaticData.bRelativePathToRootInitialized)
	{
		FString RootDirectory = FPaths::RootDir();
		FString BaseDirectory = FPlatformProcess::BaseDir();

		// this is how to go from the base dir back to the root
		StaticData.RelativePathToRoot = RootDirectory;
		FPaths::MakePathRelativeTo(StaticData.RelativePathToRoot, *BaseDirectory);

		// Ensure that the path ends w/ '/'
		if ((StaticData.RelativePathToRoot.Len() > 0) && (StaticData.RelativePathToRoot.EndsWith(TEXT("/"), ESearchCase::CaseSensitive) == false) && (StaticData.RelativePathToRoot.EndsWith(TEXT("\\"), ESearchCase::CaseSensitive) == false))
		{
			StaticData.RelativePathToRoot += TEXT("/");
		}

		StaticData.bRelativePathToRootInitialized = true;
	}

	return StaticData.RelativePathToRoot;
}

FString FPaths::CombineInternal(const FStringView* Paths, int32 NumPaths)
{
	check(Paths && NumPaths > 0);

	int32 CombinedPathLen = 0;
	for (int32 i=0; i < NumPaths; ++i)
	{
		CombinedPathLen += Paths[i].Len() + 1;
	}

	FString CombinedPath;
	CombinedPath.Reserve(CombinedPathLen);
	CombinedPath += Paths[0];
	
	for (int32 i=1; i < NumPaths; ++i)
	{
		CombinedPath /= Paths[i];
	}

	return CombinedPath;
}

bool FPaths::IsSamePath(const FString& PathA, const FString& PathB)
{
	FString TmpA = FPaths::ConvertRelativePathToFull(PathA);
	FString TmpB = FPaths::ConvertRelativePathToFull(PathB);

	FPaths::RemoveDuplicateSlashes(TmpA);
	FPaths::RemoveDuplicateSlashes(TmpB);

#if PLATFORM_MICROSOFT
	return FCString::Stricmp(*TmpA, *TmpB) == 0;
#else
	return FCString::Strcmp(*TmpA, *TmpB) == 0;
#endif
}

bool FPaths::IsUnderDirectory(const FString& InPath, const FString& InDirectory)
{
	FString Path = FPaths::ConvertRelativePathToFull(InPath);

	FString Directory = FPaths::ConvertRelativePathToFull(InDirectory);
	if (Directory.EndsWith(TEXT("/")))
	{
		Directory.RemoveAt(Directory.Len() - 1);
	}

#if PLATFORM_MICROSOFT
	int Compare = FCString::Strnicmp(*Path, *Directory, Directory.Len());
#else
	int Compare = FCString::Strncmp(*Path, *Directory, Directory.Len());
#endif

	return Compare == 0 && (Path.Len() == Directory.Len() || Path[Directory.Len()] == TEXT('/'));
}


void FPaths::TearDown()
{
	TLazySingleton<FStaticData>::TearDown();
}

const FString& FPaths::CustomUserDirArgument()
{
	FStaticData& StaticData = TLazySingleton<FStaticData>::Get();

	if (!StaticData.bUserDirArgInitialized)
	{
		// Check for a -userdir arg. If set this overrides the platform preference for using the UserDir and
		// the default. The caller is responsible for ensuring that this is a valid path for the current platform!
		FParse::Value(FCommandLine::Get(), TEXT("UserDir="), StaticData.UserDirArg);
		StaticData.bUserDirArgInitialized = true;

		if (StaticData.UserDirArg.IsEmpty() == false)
		{
			if (FPaths::IsRelative(StaticData.UserDirArg))
			{
				StaticData.UserDirArg = FPaths::Combine(*FPaths::ProjectDir(), *StaticData.UserDirArg) + TEXT("/");
			}
			else
			{
				FPaths::NormalizeDirectoryName(StaticData.UserDirArg);
				StaticData.UserDirArg = StaticData.UserDirArg + TEXT("/");
			}
		}
	}

	return StaticData.UserDirArg;
}

const FString& FPaths::CustomShaderDirArgument()
{
	FStaticData& StaticData = TLazySingleton<FStaticData>::Get();

	if (!StaticData.bShaderDirInitialized)
	{
		// Check for a -userdir arg. If set this overrides the platform preference for using the UserDir and
		// the default. The caller is responsible for ensuring that this is a valid path for the current platform!
		FParse::Value(FCommandLine::Get(), TEXT("ShaderWorkingDir="), StaticData.ShaderDir);
		StaticData.bShaderDirInitialized = true;

		if (StaticData.ShaderDir.IsEmpty() == false)
		{
			if (FPaths::IsRelative(StaticData.ShaderDir))
			{
				StaticData.ShaderDir = FPaths::Combine(*FPaths::ProjectDir(), *StaticData.ShaderDir) + TEXT("/");
			}
			else
			{
				FPaths::NormalizeDirectoryName(StaticData.ShaderDir);
				StaticData.ShaderDir = StaticData.ShaderDir + TEXT("/");
			}
		}
	}

	return StaticData.ShaderDir;
}
