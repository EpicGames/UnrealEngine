// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelFormat.h"
#include "Math/IntVector.h"

//
//	Pixel format information.
//

constexpr FPixelFormatInfo::FPixelFormatInfo(
	EPixelFormat InUnrealFormat,
	const TCHAR* InName,
	int32 InBlockSizeX,
	int32 InBlockSizeY,
	int32 InBlockSizeZ,
	int32 InBlockBytes,
	int32 InNumComponents,
	bool  InSupported)
	: Name(InName)
	, UnrealFormat(InUnrealFormat)
	, BlockSizeX(InBlockSizeX)
	, BlockSizeY(InBlockSizeY)
	, BlockSizeZ(InBlockSizeZ)
	, BlockBytes(InBlockBytes)
	, NumComponents(InNumComponents)
	, Supported(InSupported)
	, bIs24BitUnormDepthStencil(true)
{
}

FPixelFormatInfo    GPixelFormats[PF_MAX] =
{
	//               UnrealFormat           Name                  BlockSizeX  BlockSizeY  BlockSizeZ  BlockBytes  NumComponents    Supported
	FPixelFormatInfo(PF_Unknown,            TEXT("unknown"),               0,          0,          0,          0,          0,              0),
	FPixelFormatInfo(PF_A32B32G32R32F,      TEXT("A32B32G32R32F"),         1,          1,          1,          16,         4,              1),
	FPixelFormatInfo(PF_B8G8R8A8,           TEXT("B8G8R8A8"),              1,          1,          1,          4,          4,              1),
	FPixelFormatInfo(PF_G8,                 TEXT("G8"),                    1,          1,          1,          1,          1,              1),
	FPixelFormatInfo(PF_G16,                TEXT("G16"),                   1,          1,          1,          2,          1,              1),
	FPixelFormatInfo(PF_DXT1,               TEXT("DXT1"),                  4,          4,          1,          8,          3,              1),
	FPixelFormatInfo(PF_DXT3,               TEXT("DXT3"),                  4,          4,          1,          16,         4,              1),
	FPixelFormatInfo(PF_DXT5,               TEXT("DXT5"),                  4,          4,          1,          16,         4,              1),
	FPixelFormatInfo(PF_UYVY,               TEXT("UYVY"),                  2,          1,          1,          4,          4,              0),
	FPixelFormatInfo(PF_FloatRGB,           TEXT("FloatRGB"),              1,          1,          1,          4,          3,              1),
	FPixelFormatInfo(PF_FloatRGBA,          TEXT("FloatRGBA"),             1,          1,          1,          8,          4,              1),
	FPixelFormatInfo(PF_DepthStencil,       TEXT("DepthStencil"),          1,          1,          1,          4,          1,              0),
	FPixelFormatInfo(PF_ShadowDepth,        TEXT("ShadowDepth"),           1,          1,          1,          4,          1,              0),
	FPixelFormatInfo(PF_R32_FLOAT,          TEXT("R32_FLOAT"),             1,          1,          1,          4,          1,              1),
	FPixelFormatInfo(PF_G16R16,             TEXT("G16R16"),                1,          1,          1,          4,          2,              1),
	FPixelFormatInfo(PF_G16R16F,            TEXT("G16R16F"),               1,          1,          1,          4,          2,              1),
	FPixelFormatInfo(PF_G16R16F_FILTER,     TEXT("G16R16F_FILTER"),        1,          1,          1,          4,          2,              1),
	FPixelFormatInfo(PF_G32R32F,            TEXT("G32R32F"),               1,          1,          1,          8,          2,              1),
	FPixelFormatInfo(PF_A2B10G10R10,        TEXT("A2B10G10R10"),           1,          1,          1,          4,          4,              1),
	FPixelFormatInfo(PF_A16B16G16R16,       TEXT("A16B16G16R16"),          1,          1,          1,          8,          4,              1),
	FPixelFormatInfo(PF_D24,                TEXT("D24"),                   1,          1,          1,          4,          1,              1),
	FPixelFormatInfo(PF_R16F,               TEXT("PF_R16F"),               1,          1,          1,          2,          1,              1),
	FPixelFormatInfo(PF_R16F_FILTER,        TEXT("PF_R16F_FILTER"),        1,          1,          1,          2,          1,              1),
	FPixelFormatInfo(PF_BC5,                TEXT("BC5"),                   4,          4,          1,          16,         2,              1),
	FPixelFormatInfo(PF_V8U8,               TEXT("V8U8"),                  1,          1,          1,          2,          2,              1),
	FPixelFormatInfo(PF_A1,                 TEXT("A1"),                    1,          1,          1,          1,          1,              0),
	FPixelFormatInfo(PF_FloatR11G11B10,     TEXT("FloatR11G11B10"),        1,          1,          1,          4,          3,              0),
	FPixelFormatInfo(PF_A8,                 TEXT("A8"),                    1,          1,          1,          1,          1,              1),
	FPixelFormatInfo(PF_R32_UINT,           TEXT("R32_UINT"),              1,          1,          1,          4,          1,              1),
	FPixelFormatInfo(PF_R32_SINT,           TEXT("R32_SINT"),              1,          1,          1,          4,          1,              1),

	// IOS Support
	FPixelFormatInfo(PF_PVRTC2,             TEXT("PVRTC2"),                8,          4,          1,          8,          4,              0),
	FPixelFormatInfo(PF_PVRTC4,             TEXT("PVRTC4"),                4,          4,          1,          8,          4,              0),

	FPixelFormatInfo(PF_R16_UINT,           TEXT("R16_UINT"),              1,          1,          1,          2,          1,              1),
	FPixelFormatInfo(PF_R16_SINT,           TEXT("R16_SINT"),              1,          1,          1,          2,          1,              1),
	FPixelFormatInfo(PF_R16G16B16A16_UINT,  TEXT("R16G16B16A16_UINT"),     1,          1,          1,          8,          4,              1),
	FPixelFormatInfo(PF_R16G16B16A16_SINT,  TEXT("R16G16B16A16_SINT"),     1,          1,          1,          8,          4,              1),
	FPixelFormatInfo(PF_R5G6B5_UNORM,       TEXT("R5G6B5_UNORM"),          1,          1,          1,          2,          3,              0),
	FPixelFormatInfo(PF_R8G8B8A8,           TEXT("R8G8B8A8"),              1,          1,          1,          4,          4,              1),
	FPixelFormatInfo(PF_A8R8G8B8,           TEXT("A8R8G8B8"),              1,          1,          1,          4,          4,              1),
	FPixelFormatInfo(PF_BC4,                TEXT("BC4"),                   4,          4,          1,          8,          1,              1),
	FPixelFormatInfo(PF_R8G8,               TEXT("R8G8"),                  1,          1,          1,          2,          2,              1),

	FPixelFormatInfo(PF_ATC_RGB,            TEXT("ATC_RGB"),               4,          4,          1,          8,          3,              0),
	FPixelFormatInfo(PF_ATC_RGBA_E,         TEXT("ATC_RGBA_E"),            4,          4,          1,          16,         4,              0),
	FPixelFormatInfo(PF_ATC_RGBA_I,         TEXT("ATC_RGBA_I"),            4,          4,          1,          16,         4,              0),
	FPixelFormatInfo(PF_X24_G8,             TEXT("X24_G8"),                1,          1,          1,          1,          1,              0),
	FPixelFormatInfo(PF_ETC1,               TEXT("ETC1"),                  4,          4,          1,          8,          3,              0),
	FPixelFormatInfo(PF_ETC2_RGB,           TEXT("ETC2_RGB"),              4,          4,          1,          8,          3,              0),
	FPixelFormatInfo(PF_ETC2_RGBA,          TEXT("ETC2_RGBA"),             4,          4,          1,          16,         4,              0),
	FPixelFormatInfo(PF_R32G32B32A32_UINT,  TEXT("PF_R32G32B32A32_UINT"),  1,          1,          1,          16,         4,              1),
	FPixelFormatInfo(PF_R16G16_UINT,        TEXT("PF_R16G16_UINT"),        1,          1,          1,          4,          4,              1),

	// ASTC support
	FPixelFormatInfo(PF_ASTC_4x4,           TEXT("ASTC_4x4"),              4,          4,          1,          16,         4,              0),
	FPixelFormatInfo(PF_ASTC_6x6,           TEXT("ASTC_6x6"),              6,          6,          1,          16,         4,              0),
	FPixelFormatInfo(PF_ASTC_8x8,           TEXT("ASTC_8x8"),              8,          8,          1,          16,         4,              0),
	FPixelFormatInfo(PF_ASTC_10x10,         TEXT("ASTC_10x10"),            10,         10,         1,          16,         4,              0),
	FPixelFormatInfo(PF_ASTC_12x12,         TEXT("ASTC_12x12"),            12,         12,         1,          16,         4,              0),
	
	FPixelFormatInfo(PF_BC6H,               TEXT("BC6H"),                  4,          4,          1,          16,         3,              1),
	FPixelFormatInfo(PF_BC7,                TEXT("BC7"),                   4,          4,          1,          16,         4,              1),
	FPixelFormatInfo(PF_R8_UINT,            TEXT("R8_UINT"),               1,          1,          1,          1,          1,              1),
	FPixelFormatInfo(PF_L8,                 TEXT("L8"),                    1,          1,          1,          1,          1,              0),
	FPixelFormatInfo(PF_XGXR8,              TEXT("XGXR8"),                 1,          1,          1,          4,          4,              1),
	FPixelFormatInfo(PF_R8G8B8A8_UINT,      TEXT("R8G8B8A8_UINT"),         1,          1,          1,          4,          4,              1),
	FPixelFormatInfo(PF_R8G8B8A8_SNORM,     TEXT("R8G8B8A8_SNORM"),        1,          1,          1,          4,          4,              1),

	FPixelFormatInfo(PF_R16G16B16A16_UNORM, TEXT("R16G16B16A16_UINT"),     1,          1,          1,          8,          4,              1),
	FPixelFormatInfo(PF_R16G16B16A16_SNORM, TEXT("R16G16B16A16_SINT"),     1,          1,          1,          8,          4,              1),
	FPixelFormatInfo(PF_PLATFORM_HDR_0,     TEXT("PLATFORM_HDR_0"),        0,          0,          0,          0,          0,              0),
	FPixelFormatInfo(PF_PLATFORM_HDR_1,     TEXT("PLATFORM_HDR_1"),        0,          0,          0,          0,          0,              0),
	FPixelFormatInfo(PF_PLATFORM_HDR_2,     TEXT("PLATFORM_HDR_2"),        0,          0,          0,          0,          0,              0),

	// NV12 contains 2 textures: R8 luminance plane followed by R8G8 1/4 size chrominance plane.
	// BlockSize/BlockBytes/NumComponents values don't make much sense for this format, so set them all to one.
	FPixelFormatInfo(PF_NV12,               TEXT("NV12"),                  1,          1,          1,          1,          1,              0),

	FPixelFormatInfo(PF_R32G32_UINT,        TEXT("PF_R32G32_UINT"),        1,          1,          1,          8,          2,              1),

	FPixelFormatInfo(PF_ETC2_R11_EAC,       TEXT("PF_ETC2_R11_EAC"),       4,          4,          1,          8,          1,              0),
	FPixelFormatInfo(PF_ETC2_RG11_EAC,      TEXT("PF_ETC2_RG11_EAC"),      4,          4,          1,          16,         2,              0),
	FPixelFormatInfo(PF_R8,                 TEXT("R8"),                    1,          1,          1,          1,          1,              1),
    FPixelFormatInfo(PF_B5G5R5A1_UNORM,     TEXT("B5G5R5A1_UNORM"),        1,          1,          1,          2,          4,              0),

	// ASTC HDR support
	FPixelFormatInfo(PF_ASTC_4x4_HDR,       TEXT("ASTC_4x4_HDR"),          4,          4,          1,          16,         4,              0),
	FPixelFormatInfo(PF_ASTC_6x6_HDR,       TEXT("ASTC_6x6_HDR"),          6,          6,          1,          16,         4,              0),
	FPixelFormatInfo(PF_ASTC_8x8_HDR,       TEXT("ASTC_8x8_HDR"),          8,          8,          1,          16,         4,              0),
	FPixelFormatInfo(PF_ASTC_10x10_HDR,     TEXT("ASTC_10x10_HDR"),        10,         10,         1,          16,         4,              0),
	FPixelFormatInfo(PF_ASTC_12x12_HDR,     TEXT("ASTC_12x12_HDR"),        12,         12,         1,          16,         4,              0),

	FPixelFormatInfo(PF_G16R16_SNORM,       TEXT("G16R16_SNORM"),          1,          1,          1,          4,          2,              1),
	FPixelFormatInfo(PF_R8G8_UINT,          TEXT("R8G8_UINT"),             1,          1,          1,          2,          2,              1),
	FPixelFormatInfo(PF_R32G32B32_UINT,     TEXT("R32G32B32_UINT"),        1,          1,          1,          12,         3,              1),
	FPixelFormatInfo(PF_R32G32B32_SINT,     TEXT("R32G32B32_SINT"),        1,          1,          1,          12,         3,              1),
	FPixelFormatInfo(PF_R32G32B32F,         TEXT("R32G32B32F"),            1,          1,          1,          12,         3,              1),
	FPixelFormatInfo(PF_R8_SINT,            TEXT("R8_SINT"),               1,          1,          1,          1,          1,              1),
	FPixelFormatInfo(PF_R64_UINT,			TEXT("R64_UINT"),              1,          1,          1,          8,          1,              0),
	FPixelFormatInfo(PF_R9G9B9EXP5,			TEXT("R9G9B9EXP5"),			   1,		   1,		   1,		   4,		   4,			   0),

	// P010 contains 2 textures: R16 luminance plane followed by R16G16 1/4 size chrominance plane. (upper 10 bits used)
	// BlockSize/BlockBytes/NumComponents values don't make much sense for this format, so set them all to one.
	FPixelFormatInfo(PF_P010,				TEXT("P010"),				   1,		   1,		   1,		   2,		   1,			   0),

	// ASTC high precision NormalRG support
	FPixelFormatInfo(PF_ASTC_4x4_NORM_RG,   TEXT("ASTC_4x4_NORM_RG"),      4,          4,          1,          16,         2,              0),
	FPixelFormatInfo(PF_ASTC_6x6_NORM_RG,   TEXT("ASTC_6x6_NORM_RG"),      6,          6,          1,          16,         2,              0),
	FPixelFormatInfo(PF_ASTC_8x8_NORM_RG,   TEXT("ASTC_8x8_NORM_RG"),      8,          8,          1,          16,         2,              0),
	FPixelFormatInfo(PF_ASTC_10x10_NORM_RG, TEXT("ASTC_10x10_NORM_RG"),    10,         10,         1,          16,         2,              0),
	FPixelFormatInfo(PF_ASTC_12x12_NORM_RG, TEXT("ASTC_12x12_NORM_RG"),    12,         12,         1,          16,         2,              0),

	FPixelFormatInfo(PF_R16G16_SINT,        TEXT("PF_R16G16_SINT"),        1,          1,          1,          4,          4,              1),
	FPixelFormatInfo(PF_R8G8B8,             TEXT("R8G8B8"),                1,          1,          1,          3,          3,              0),
};


uint64 FPixelFormatInfo::Get3DImageSizeInBytes(uint32 InWidth, uint32 InHeight, uint32 InDepth) const
{
	const uint64 WidthInBlocks = GetBlockCountForWidth(InWidth);
	const uint64 HeightInBlocks = GetBlockCountForHeight(InHeight);

	// no format currently has block requirement for depth
	return InDepth * WidthInBlocks * HeightInBlocks * BlockBytes;
}

uint64 FPixelFormatInfo::Get3DTextureMipSizeInBytes(uint32 InTextureWidth, uint32 InTextureHeight, uint32 InTextureDepth, uint32 InMipIndex) const
{
	int32 MipWidth = FMath::Max((int32)InTextureWidth >> InMipIndex, 1);
	int32 MipHeight = FMath::Max((int32)InTextureHeight >> InMipIndex, 1);
	int32 MipDepth = FMath::Max((int32)InTextureDepth >> InMipIndex, 1);	
	return Get3DImageSizeInBytes(MipWidth, MipHeight, MipDepth);
}

uint64 FPixelFormatInfo::Get3DTextureSizeInBytes(uint32 InTextureWidth, uint32 InTextureHeight, uint32 InTextureDepth, uint32 InMipCount) const
{
	uint64 Size = 0;
	int32 MipWidth = InTextureWidth;
	int32 MipHeight = InTextureHeight;
	int32 MipDepth = InTextureDepth;
	for (uint32 MipIndex = 0; MipIndex < InMipCount; ++MipIndex)
	{
		Size += Get3DImageSizeInBytes(MipWidth, MipHeight, MipDepth);
		MipWidth = FMath::Max(MipWidth >> 1, 1);
		MipHeight = FMath::Max(MipHeight >> 1, 1);
		MipDepth = FMath::Max(MipDepth >> 1, 1);
	}
	return Size;
}

uint64 FPixelFormatInfo::GetBlockCountForWidth(uint32 InWidth) const
{
	// note: FTexture2DResource applies a 2 block min for PVRTC that is not applied here
	if (BlockSizeX > 0)
	{
		return (InWidth + BlockSizeX - 1) / BlockSizeX;
	}
	else
	{
		return 0;
	}
}


uint64 FPixelFormatInfo::GetBlockCountForHeight(uint32 InHeight) const
{
	// note: FTexture2DResource applies a 2 block min for PVRTC that is not applied here
	if (BlockSizeY > 0)
	{
		return (InHeight + BlockSizeY - 1) / BlockSizeY;
	}
	else
	{
		return 0;
	}
}

uint64 FPixelFormatInfo::Get2DImageSizeInBytes(uint32 InWidth, uint32 InHeight) const
{
	const uint64 WidthInBlocks = GetBlockCountForWidth(InWidth);
	const uint64 HeightInBlocks = GetBlockCountForHeight(InHeight);
	return WidthInBlocks * HeightInBlocks * BlockBytes;
}

uint64 FPixelFormatInfo::Get2DTextureMipSizeInBytes(uint32 InTextureWidth, uint32 InTextureHeight, uint32 InMipIndex) const
{
	int32 MipWidth = FMath::Max((int32)InTextureWidth >> InMipIndex, 1);
	int32 MipHeight = FMath::Max((int32)InTextureHeight >> InMipIndex, 1);
	return Get2DImageSizeInBytes(MipWidth, MipHeight);
}

uint64 FPixelFormatInfo::Get2DTextureSizeInBytes(uint32 InTextureWidth, uint32 InTextureHeight, uint32 InMipCount) const
{
	uint64 Size = 0;
	int32 MipWidth = InTextureWidth;
	int32 MipHeight = InTextureHeight;
	for ( uint32 MipIndex=0; MipIndex < InMipCount; ++MipIndex )
	{
		Size += Get2DImageSizeInBytes(MipWidth, MipHeight);
		MipWidth = FMath::Max(MipWidth >> 1, 1);
		MipHeight = FMath::Max(MipHeight >> 1, 1);
	}
	return Size;
}


/** Helper functions for text output of texture properties... */
#ifndef CASE_ENUM_TO_TEXT
#define CASE_ENUM_TO_TEXT(txt) case txt: return TEXT(#txt);
#endif

#ifndef TEXT_TO_ENUM
#define TEXT_TO_ENUM(eVal, txt)		if (FCString::Stricmp(TEXT(#eVal), txt) == 0)	return eVal;
#endif

const TCHAR* GetPixelFormatString(EPixelFormat InPixelFormat)
{
	switch (InPixelFormat)
	{
		FOREACH_ENUM_EPIXELFORMAT_CORE(CASE_ENUM_TO_TEXT)
	default:
		return TEXT("PF_Unknown");
	}
}

EPixelFormat GetPixelFormatFromString(const TCHAR* InPixelFormatStr)
{
#define TEXT_TO_PIXELFORMAT(f) TEXT_TO_ENUM(f, InPixelFormatStr);
	FOREACH_ENUM_EPIXELFORMAT_CORE(TEXT_TO_PIXELFORMAT)
#undef TEXT_TO_PIXELFORMAT
		return PF_Unknown;
}

static struct FValidatePixelFormats
{
	FValidatePixelFormats()
	{
		for (int32 Index = 0; Index < UE_ARRAY_COUNT(GPixelFormats); ++Index)
		{
			// Make sure GPixelFormats has an entry for every unreal format
			checkf((EPixelFormat)Index == GPixelFormats[Index].UnrealFormat, TEXT("Missing entry for EPixelFormat %d"), (int32)Index);
		}
	}
} ValidatePixelFormats;
