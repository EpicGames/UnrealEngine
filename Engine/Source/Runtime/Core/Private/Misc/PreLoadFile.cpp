// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/PreLoadFile.h"
#include "HAL/Event.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"
#include "Async/AsyncFileHandle.h"
#include "Misc/Paths.h"
#include "Misc/ScopeLock.h"

bool FPreLoadFile::bSystemNoLongerTakingRequests = false;

//
struct FPreloadFileRegistry
{
	FCriticalSection FileLock;
	TMap<FString, FPreLoadFile*> Files;

	static FPreloadFileRegistry& Get()
	{
		static FPreloadFileRegistry Instance;
		return Instance;
	}
};

#ifndef PLATFORM_CAN_ASYNC_PRELOAD_FILES
	#define PLATFORM_CAN_ASYNC_PRELOAD_FILES 0
#endif

FPreLoadFile::FPreLoadFile(const TCHAR* InPath)
	: FDelayedAutoRegisterHelper([] { if constexpr (STATS) { return EDelayedRegisterRunPhase::StatSystemReady; } else { return EDelayedRegisterRunPhase::FileSystemReady; }}(), [this] {
		bSystemNoLongerTakingRequests = false;
#if PLATFORM_CAN_ASYNC_PRELOAD_FILES
		KickOffRead();
#endif
	})
	, bIsComplete(false)
	, bFailedToOpenInKickOff(false)
	, Data(nullptr)
	, FileSize(0)
	, Path(InPath)
	, CompletionEvent(nullptr)
{
	checkf(bSystemNoLongerTakingRequests == false, TEXT("Created a PreLoadFile object after it is no longer valid"));
	FPreloadFileRegistry& Registry = FPreloadFileRegistry::Get();
	FScopeLock Lock(&Registry.FileLock);
	Registry.Files.Add(Path, this);
}

void FPreLoadFile::KickOffRead()
{
	SCOPED_BOOT_TIMING("FPreLoadFile::KickOffRead");

	if (Path.StartsWith(TEXT("{PROJECT}")))
	{
		Path = Path.Replace(TEXT("{PROJECT}"), *FPaths::ProjectDir());
	}

	// if we are reading again because we had failed earlier, the CompletionEvent may be valid, no need to recreate it
	if (CompletionEvent == nullptr)
	{
		CompletionEvent = FPlatformProcess::GetSynchEventFromPool();
		check(CompletionEvent != nullptr);
	}

#if PLATFORM_CAN_ASYNC_PRELOAD_FILES
	FAsyncFileCallBack SizeCallbackFunction = [this](bool bWasCancelled, IAsyncReadRequest* SizeRequest)
	{
		FileSize = (int32)SizeRequest->GetSizeResults();

		if (FileSize > 0)
		{
			FAsyncFileCallBack ReadCallbackFunction = [this](bool bWasCancelledInner, IAsyncReadRequest* ReadRequest)
			{
				Data = ReadRequest->GetReadResults();
				CompletionEvent->Trigger();
			};
			AsyncReadHandle->ReadRequest(0, FileSize, EAsyncIOPriorityAndFlags::AIOP_High, &ReadCallbackFunction);
		}
		else
		{
			// it's possible pak files with the file weren't mounted yet, so note that we didn't find it, and try again in TakeOwnership time
			bFailedToOpenInKickOff = true;

			FileSize = -1;
			CompletionEvent->Trigger();
		}
	};

	AsyncReadHandle = FPlatformFileManager::Get().GetPlatformFile().OpenAsyncRead(*Path);
	SizeRequestHandle = AsyncReadHandle->SizeRequest(&SizeCallbackFunction);

#else

	FArchive* Reader = IFileManager::Get().CreateFileReader(*Path);
	if (Reader != nullptr)
	{
		FileSize = Reader->TotalSize();
		Data = FMemory::Malloc(FileSize);
		Reader->Serialize(Data, FileSize);
		delete Reader;
		CompletionEvent->Trigger();
	}
	else
	{
		// if we are failing a second time, then just mark the event complete, we have exhausted attempts
//		if (bFailedToOpenInKickOff)
		{
			CompletionEvent->Trigger();
		}
		// it's possible pak files with the file weren't mounted yet, so note that we didn't find it, and try again in TakeOwnership time
		bFailedToOpenInKickOff = true;
	}
#endif
}	


void* FPreLoadFile::TakeOwnershipOfLoadedData(int64* OutFileSize)
{
	if (CompletionEvent)
	{
		if (CompletionEvent->Wait(0) == false)
		{
			// wait until we are done
			CompletionEvent->Wait();
		}
	}

	// may need to attempt to read again, if (re-)requesting data after the initial boot sequence, or if a pak file wasn't mounted in time
	if (!CompletionEvent || bFailedToOpenInKickOff)
	{
		check(CompletionEvent == nullptr || !CompletionEvent->IsManualReset());

		KickOffRead();

		check(CompletionEvent); // KickOffRead() should make a completion event
		if (CompletionEvent->Wait(0) == false)
		{
			// wait until we are done
			CompletionEvent->Wait();
		}
	}

#if PLATFORM_CAN_ASYNC_PRELOAD_FILES
	delete SizeRequestHandle;
	delete AsyncReadHandle;
	SizeRequestHandle = nullptr;
	AsyncReadHandle = nullptr;
#endif

	FPlatformProcess::ReturnSynchEventToPool(CompletionEvent);
	CompletionEvent = nullptr;

	// return the size of Data if needed
	if (OutFileSize)
	{
		*OutFileSize = FileSize;
	}

	void* ReturnData = Data;
	Data = nullptr;

	// remove ourself from registry
	{
		FPreloadFileRegistry& Registry = FPreloadFileRegistry::Get();
		FScopeLock Lock(&Registry.FileLock);
		Registry.Files.Remove(Path);
	}

	return ReturnData;
}

void* FPreLoadFile::TakeOwnershipOfLoadedDataByPath(const TCHAR* Filename, int64* OutFileSize)
{
	FPreLoadFile* ExistingPreLoad;
	{
		FPreloadFileRegistry& Registry = FPreloadFileRegistry::Get();
		FScopeLock Lock(&Registry.FileLock);
		ExistingPreLoad = Registry.Files.FindRef(Filename);
	}

	// if we never attempted to read this, return 0 as filesize
	if (ExistingPreLoad == nullptr)
	{
		*OutFileSize = 0;
		return nullptr;

	}
	return ExistingPreLoad->TakeOwnershipOfLoadedData(OutFileSize);
}
