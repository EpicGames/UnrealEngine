// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/PreloadableFile.h"

#include "Async/Async.h"
#include "Async/AsyncFileHandle.h"
#include "GenericPlatform/GenericPlatformMisc.h"
#include "HAL/Event.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/Paths.h"
#include "Misc/ScopeLock.h"
#include "Math/NumericLimits.h"
#include "ProfilingDebugging/CookStats.h"
#include "ProfilingDebugging/ScopedTimers.h"

#define PRELOADABLEFILE_COOK_STATS_ENABLED 0 && ENABLE_COOK_STATS

#if PRELOADABLEFILE_COOK_STATS_ENABLED
namespace FPreloadableArchiveImpl
{
	static int64 NumNonPreloadedPages = 0;
	static int64 NumPreloadedPages = 0;
	static double SerializeTime = 0;
	static double OpenFileTime = 0;
	FCriticalSection OpenFileTimeLock;
	static FCookStatsManager::FAutoRegisterCallback RegisterCookStats([](FCookStatsManager::AddStatFuncRef AddStat)
		{
			AddStat(TEXT("Package.Load"), FCookStatsManager::CreateKeyValueArray(TEXT("NumPreloadedPages"), NumPreloadedPages));
			AddStat(TEXT("Package.Load"), FCookStatsManager::CreateKeyValueArray(TEXT("NumNonPreloadedPages"), NumNonPreloadedPages));
			AddStat(TEXT("Package.Load"), FCookStatsManager::CreateKeyValueArray(TEXT("SerializeTime"), SerializeTime));
			AddStat(TEXT("Package.Load"), FCookStatsManager::CreateKeyValueArray(TEXT("OpenFileTime"), OpenFileTime));
		});
}
#endif

FPreloadableArchive::FPreloadableArchive(FStringView InArchiveName)
	: FArchive()
	, ArchiveName(InArchiveName)
	, bInitialized(false)
	, bIsPreloading(false)
	, bIsPreloadingPaused(false)
	, CacheEnd(0)
{
	PendingAsyncComplete = FPlatformProcess::GetSynchEventFromPool(true);
	PendingAsyncComplete->Trigger();
	this->SetIsLoading(true);
	this->SetIsPersistent(true);
}

FPreloadableArchive::~FPreloadableArchive()
{
	Close();
	// It is possible we set a flag indicating that an async event is done, but we haven't yet called Trigger in the task thread; Trigger is always the last memory-accessing instruction on the task thread
	// This happens for example at the end of FPreloadableArchive::InitializeInternal
	// Wait for the trigger call to be made before deleting PendingAsyncComplete.
	PendingAsyncComplete->Wait();
	FPlatformProcess::ReturnSynchEventToPool(PendingAsyncComplete);
}

void FPreloadableArchive::SetPageSize(int64 InPageSize)
{
	if (!bInitialized)
	{
		PendingAsyncComplete->Wait();
	}
	if (bInitialized)
	{
		checkf(!bInitialized, TEXT("It is invalid to SetPageSize after initialization"));
		return;
	}
	PageSize = InPageSize;
}

void FPreloadableArchive::InitializeAsync(FCreateArchive&& InCreateArchiveFunction, uint32 InFlags, int64 PrimeSize)
{
	InFlags = (InFlags & (~Flags::ModeBits)) | Flags::PreloadHandle;
	InitializeInternalAsync(MoveTemp(InCreateArchiveFunction), FCreateAsyncArchive(), InFlags, PrimeSize);
}

void FPreloadableArchive::InitializeAsync(FCreateAsyncArchive&& InCreateAsyncArchiveFunction, uint32 InFlags, int64 PrimeSize)
{
	InFlags = (InFlags & (~Flags::ModeBits)) | Flags::PreloadBytes;
	InitializeInternalAsync(FCreateArchive(), MoveTemp(InCreateAsyncArchiveFunction), InFlags, PrimeSize);
}

bool FPreloadableArchive::IsInitialized() const
{
	return bInitialized;
}

void FPreloadableArchive::WaitForInitialization() const
{
	if (bInitialized)
	{
		return;
	}
	PendingAsyncComplete->Wait();
}

void FPreloadableArchive::InitializeInternalAsync(FCreateArchive&& InCreateArchiveFunction, FCreateAsyncArchive&& InCreateAsyncArchiveFunction, uint32 InFlags, int64 PrimeSize)
{
	if (!bInitialized)
	{
		PendingAsyncComplete->Wait();
	}
	if (bInitialized)
	{
		return;
	}
	check(PendingAsyncComplete->Wait());
	PendingAsyncComplete->Reset();
	Async(EAsyncExecution::TaskGraph, [this, InFlags, PrimeSize, PassedArchiveFunction = MoveTemp(InCreateArchiveFunction), PassedAsyncArchiveFunction = MoveTemp(InCreateAsyncArchiveFunction)]
		() mutable
		{
			InitializeInternal(MoveTemp(PassedArchiveFunction), MoveTemp(PassedAsyncArchiveFunction), InFlags, PrimeSize);
		});
}


void FPreloadableArchive::InitializeInternal(FCreateArchive&& InCreateArchiveFunction, FCreateAsyncArchive&& InCreateAsyncArchiveFunction, uint32 InFlags, int64 PrimeSize)
{
	check(!bInitialized);

	uint32 Mode = InFlags & Flags::ModeBits;
	switch (Mode)
	{
	case Flags::PreloadBytes:
	{
		if (InCreateAsyncArchiveFunction)
		{
			AsynchronousHandle.Reset(InCreateAsyncArchiveFunction());
		}
		if (AsynchronousHandle)
		{
			IAsyncReadRequest* SizeRequest = AsynchronousHandle->SizeRequest();
			if (!SizeRequest)
			{
				// AsyncReadHandle is not working; fall back to the synchronous handle
				AsynchronousHandle.Reset();
			}
			else
			{
				SizeRequest->WaitCompletion();
				Size = SizeRequest->GetSizeResults();
				delete SizeRequest;
			}
		}
		break;
	}
	case Flags::PreloadHandle:
	{
		if (InCreateArchiveFunction)
		{
#if PRELOADABLEFILE_COOK_STATS_ENABLED
			FScopeLock OpenFileTimeScopeLock(&FPreloadableArchiveImpl::OpenFileTimeLock);
			FScopedDurationTimer ScopedDurationTimer(FPreloadableArchiveImpl::OpenFileTime);
#endif
			SynchronousArchive.Reset(InCreateArchiveFunction());
		}

		if (SynchronousArchive)
		{
			Size = SynchronousArchive->TotalSize();
			if ((InFlags & (Flags::Prime)) && PrimeSize > 0)
			{
				SynchronousArchive->Precache(0, PrimeSize);
			}
		}
		break;
	}
	default:
	{
		checkf(false, TEXT("Invalid mode %u."), Mode);
		break;
	}
	}

#if FPRELOADABLEFILE_TEST_ENABLED
	if (Size != -1 && InCreateArchiveFunction)
	{
		TestArchive.Reset(InCreateArchiveFunction());
		check(TestArchive);
	}
#endif

	FPlatformMisc::MemoryBarrier(); // Make sure all members written above are fully written before we set the thread-safe variable bInitialized to true
	bInitialized = true;
	FPlatformMisc::MemoryBarrier(); // Make sure bInitialized is fully written before we wake any threads waiting on PendingAsyncComplete
	PendingAsyncComplete->Trigger();
}

bool FPreloadableArchive::StartPreload()
{
	if (bIsPreloading)
	{
		return true;
	}
	if (!bInitialized)
	{
		UE_LOGF(LogCore, Error, "Attempted FPreloadableArchive::StartPreload when uninitialized. Call will be ignored.");
		return false;
	}
	if (!AllocateCache())
	{
		return false;
	}
	// Wait for the async initialization task to complete, in case bInitialized = true was set on the InitializeAsync thread but PendingAsyncComplete->Trigger has not yet been called.
	// This might also wait for PendingAsyncComplete to finish after the last Preloading task completed, after setting bIsPreloading = false, but that's a don't-care because that
	// PendingAsyncComplete is triggered before the Preloading task thread exits the critical section that we enter below.
	PendingAsyncComplete->Wait();

	FScopeLock ScopeLock(&PreloadLock);
	bIsPreloading = true;
	check(!bIsPreloadingPaused); // IsPreloadingPaused is an internally-set value that is always reset to false before exiting from a public interface function
	ResumePreload();
	return true;
}

void FPreloadableArchive::StopPreload()
{
	if (!bIsPreloading)
	{
		FScopeLock ScopeLock(&PreloadLock);
		FreeRetiredRequests();
		return;
	}
	PausePreload();
	bIsPreloading = false;
	bIsPreloadingPaused = false;
}

bool FPreloadableArchive::IsPreloading() const
{
	// Note that this function is for public use only, and a true result does not indicate we have a currently pending Preload operation;
	// we may be paused even if bIsPreloading is true.
	return bIsPreloading;
}

bool FPreloadableArchive::AllocateCache()
{
	if (IsCacheAllocated())
	{
		return true;
	}
	if (!bInitialized)
	{
		UE_LOGF(LogCore, Error, "Attempted FPreloadableArchive::AllocateCache when uninitialized. Call will be ignored.");
		return false;
	}
	if (Size < 0)
	{
		return false;
	}
	if (!AsynchronousHandle)
	{
		return false;
	}

	check(CacheBytes == nullptr); // Otherwise IsCacheAllocated would have returned true
	CacheBytes = reinterpret_cast<uint8*>(FMemory::Malloc(FMath::Max(Size, (int64)1)));
	return true;
}

void FPreloadableArchive::ReleaseCache()
{
	if (!IsCacheAllocated())
	{
		return;
	}

	StopPreload();
#if PRELOADABLEFILE_COOK_STATS_ENABLED
	FPreloadableArchiveImpl::NumPreloadedPages += CacheEnd / PageSize;
	FPreloadableArchiveImpl::NumNonPreloadedPages += (Size - CacheEnd + PageSize - 1) / PageSize;
#endif
	FMemory::Free(CacheBytes);
	CacheBytes = nullptr;
	check(RetiredRequests.Num() == 0);
	RetiredRequests.Shrink();
}

bool FPreloadableArchive::IsCacheAllocated() const
{
	return CacheBytes != nullptr;
}

FArchive* FPreloadableArchive::DetachLowerLevel()
{
	WaitForInitialization();
	return SynchronousArchive.Release();
}

bool FPreloadableArchive::HasValidData() const
{
	WaitForInitialization();
	return SynchronousArchive || AsynchronousHandle;
}

void FPreloadableArchive::PausePreload()
{
	bIsPreloadingPaused = true;
	PendingAsyncComplete->Wait();

	{
		FScopeLock ScopeLock(&PreloadLock);
		FreeRetiredRequests();
	}
}

void FPreloadableArchive::ResumePreload()
{
	// Contract: This function is only called when inside the PreloadLock CriticalSection
	// Contract: this function is only called when already initialized and no async reads are pending
	check(PendingAsyncComplete->Wait(0));

	bIsPreloadingPaused = false;
	PendingAsyncComplete->Reset();
	bool bComplete = ResumePreloadNonRecursive();
	if (!bReadCompleteWasCalledInline)
	{
		if (bComplete)
		{
			PendingAsyncComplete->Trigger();
		}
	}
	else
	{
		check(!bComplete);
		bool bCanceled;
		IAsyncReadRequest* ReadRequest;
		SavedReadCompleteArguments.Get(bCanceled, ReadRequest);

		// This call to OnReadComplete may result in further calls to ResumePreloadNonRecursive
		OnReadComplete(bCanceled, ReadRequest);
	}
}

bool FPreloadableArchive::ResumePreloadNonRecursive()
{
	check(!PendingAsyncComplete->Wait(0)); // Caller should have set this before calling
	int64 RemainingSize = Size - CacheEnd;
	if (RemainingSize <= 0)
	{
		FPlatformMisc::MemoryBarrier(); // Make sure we have fully written any values of CacheEnd written by our caller before we set the thread-safe bIsPreloading value to false
		bIsPreloading = false;
		FPlatformMisc::MemoryBarrier(); // Our caller will call PendingAsyncComplete->Trigger; make sure that the thread-safe bIsPreloading value has been fully written before waking any threads waiting on PendingAsyncComplete
		return true;
	}
	if (bIsPreloadingPaused)
	{
		return true;
	}
	int64 ReadSize = FMath::Min(RemainingSize, PageSize);
	// If called from ResumePreload, these flags should all be false because we had no pending async call and we set them to false in the constructor or the last call to OnReadComplete
	// If called from OnReadComplete, OnReadComplete should have cleared them within the PreloadLock before calling
	check(!bIsInlineReadComplete && !bReadCompleteWasCalledInline); 
	bIsInlineReadComplete = true;
	FAsyncFileCallBack CompletionCallback = [this](bool bCanceled, IAsyncReadRequest* InRequest) { OnReadComplete(bCanceled, InRequest); };
	IAsyncReadRequest* ReadRequest = AsynchronousHandle->ReadRequest(CacheEnd, ReadSize, AIOP_Normal, &CompletionCallback, CacheBytes + CacheEnd);
	if (!ReadRequest)
	{
		// There was a bug with our request
		UE_LOGF(LogCore, Warning, "ReadRequest returned null");
		bIsInlineReadComplete = false;
		FPlatformMisc::MemoryBarrier(); // Make sure we have fully written any values of CacheEnd written by our caller before we set the thread-safe bIsPreloading variable to false
		bIsPreloading = false;
		FPlatformMisc::MemoryBarrier(); // Our caller will call PendingAsyncComplete->Trigger; make sure that the thread-safe bIsPreloading variable has been fully written before waking any threads waiting on PendingAsyncComplete
		return true;
	}
	bIsInlineReadComplete = false;
	return false;
}

void FPreloadableArchive::OnReadComplete(bool bCanceled, IAsyncReadRequest* ReadRequest)
{
	TArray<IAsyncReadRequest*> LocalRetired;
	while (true)
	{
		FScopeLock ScopeLock(&PreloadLock);
		if (bIsInlineReadComplete)
		{
			SavedReadCompleteArguments.Set(bCanceled, ReadRequest);
			bReadCompleteWasCalledInline = true;
			check(LocalRetired.Num() == 0);
			return;
		}
		bReadCompleteWasCalledInline = false;
		FreeRetiredRequests();

		// We are not allowed to delete any ReadRequests until after other work that is done on the callback thread that occurs AFTER the callback has run, such as
		//  1) FAsyncTask::FinishThreadedWork which is called from FAsyncTask::DoThreadedWork, after the call to DoWork that results in our callback being called
		//  2) AsyncReadRequest::SetAllComplete which is called from AsyncReadRequest::SetComplete, after the call to SetDataComplete that results in our callback being called
		// So instead of deleting the request now, add it to the list of RetiredRequests. Both future calls to OnReadComplete and the class teardown code (Close and PausePrecache) will
		//  A) Wait for the request to complete, in case they are run simultaneously with this call thread after we have added the request to retired but before SetAllComplete is called
		//  B) Delete the request, which will then wait for FinishThreadedWork to be called if it hasn't already.
		// We need to add to a local copy of Retired until we're ready to return from OnReadComplete, so that later iterations of this loop inside of OnReadComplete do not attempt to wait on the ReadRequests
		//   we retired in earlier iterations.
		// One unfortunate side effect of this this retirement design is that we will hang on to the IAsyncReadRequest instance for the final request until Close is called.
		LocalRetired.Add(ReadRequest);

		uint8* ReadResults = ReadRequest->GetReadResults();
		if (bCanceled || !ReadResults)
		{
			UE_LOGF(LogCore, Warning, "Precaching failed for %ls: %ls.", *GetArchiveName(), (bCanceled ? TEXT("Canceled") : TEXT("GetReadResults returned null")));
			RetiredRequests.Append(MoveTemp(LocalRetired));
			FPlatformMisc::MemoryBarrier(); // Make sure we have fully written any values of CacheEnd written earlier in the loop before we set the thread-safe bIsPreloading value to false
			bIsPreloading = false;
			FPlatformMisc::MemoryBarrier(); // Make sure we have fully written bIsPreloading before we wake any threads waiting on PendingAsyncComplete
			PendingAsyncComplete->Trigger();
			return;
		}
		else
		{
			check(ReadResults == CacheBytes + CacheEnd);
			int64 ReadSize = FMath::Min(PageSize, Size - CacheEnd);
			FPlatformMisc::MemoryBarrier(); // Make sure we set have written the bytes (which our caller did at some point before calling OnReadComplete) before we increment the readonly-threadsafe variable CacheEnd
			CacheEnd += ReadSize;
			bool bComplete = ResumePreloadNonRecursive();
			if (!bReadCompleteWasCalledInline)
			{
				RetiredRequests.Append(MoveTemp(LocalRetired));
				if (bComplete)
				{
					PendingAsyncComplete->Trigger();
				}
				return;
			}
			else
			{
				check(!bComplete);
				// PrecacheNextPage's ReadRequest completed immediately, and called OnReadComplete on the current thread.
				// We made a design decision to not allow that inner OnReadComplete to recursively execute, as that could lead
				// to a stack of arbitrary depth if all the requested pages are already precached. Instead, we saved the calling values and returned.
				// Now that we have popped up to the outer OnReadComplete, use those saved values and run again.
				SavedReadCompleteArguments.Get(bCanceled, ReadRequest);
			}
		}
	}
}

void FPreloadableArchive::FreeRetiredRequests()
{
	for (IAsyncReadRequest* Retired : RetiredRequests)
	{
		Retired->WaitCompletion();
		delete Retired;
	}
	RetiredRequests.Reset();
}


void FPreloadableArchive::Serialize(void* V, int64 Length)
{
#if PRELOADABLEFILE_COOK_STATS_ENABLED
	FScopedDurationTimer ScopeTimer(FPreloadableArchiveImpl::SerializeTime);
#endif
#if FPRELOADABLEFILE_TEST_ENABLED
	if (!TestArchive)
	{
		SerializeInternal(V, Length);
		return;
	}

	int64 SavedPos = Pos;

	bool bWasPreloading = IsPreloading();
	TestArchive->Seek(Pos);
	TArray64<uint8> TestBytes;
	TestBytes.AddUninitialized(Length);
	TestArchive->Serialize(TestBytes.GetData(), Length);

	SerializeInternal(V, Length);

	bool bBytesMatch = FMemory::Memcmp(V, TestBytes.GetData(), Length) == 0;
	bool bPosMatch = Pos == TestArchive->Tell();
	if (!bBytesMatch || !bPosMatch)
	{
		UE_LOGF(LogCore, Warning, "FPreloadableArchive::Serialize Mismatch on %ls. BytesMatch=%ls, PosMatch=%ls, WasPreloading=%ls",
			*GetArchiveName(), (bBytesMatch ? TEXT("true") : TEXT("false")), (bPosMatch ? TEXT("true") : TEXT("false")),
			(bWasPreloading ? TEXT("true") : TEXT("false")));
		Seek(SavedPos);
		TestArchive->Seek(SavedPos);
		SerializeInternal(V, Length);
		TestArchive->Serialize(TestBytes.GetData(), Length);
	}
}

void FPreloadableArchive::SerializeInternal(void* V, int64 Length)
{
#endif
	if (!bInitialized)
	{
		SetError();
		UE_LOGF(LogCore, Error, "Attempted to Serialize from FPreloadableArchive when not initialized.");
		return;
	}
	if (Pos + Length > Size)
	{
		SetError();
		UE_LOG(LogCore, Error, TEXT("Requested read of %" INT64_FMT " bytes when %" INT64_FMT " bytes remain (file=%s, size=%" INT64_FMT ")"), Length, Size - Pos, *GetArchiveName(), Size);
		return;
	}

	if (!IsCacheAllocated())
	{
		SerializeSynchronously(V, Length);
		return;
	}

	bool bLocalIsPreloading = bIsPreloading;
	int64 LocalCacheEnd = CacheEnd;
	int64 EndPos = Pos + Length;
	while (Pos < EndPos)
	{
		if (LocalCacheEnd > Pos)
		{
			int64 ReadLength = FMath::Min(LocalCacheEnd, EndPos) - Pos;
			FMemory::Memcpy(V, CacheBytes + Pos, ReadLength);
			V = ((uint8*)V) + ReadLength;
			Pos += ReadLength;
		}
		else
		{
			if (bLocalIsPreloading)
			{
				PausePreload();
				check(PendingAsyncComplete->Wait(0));
				LocalCacheEnd = CacheEnd;
				if (LocalCacheEnd > Pos)
				{
					// The page we just finished Preloading contains the position. Resume Preloading and continue in the loop to read from the now-available page.
					FScopeLock ScopeLock(&PreloadLock);
					ResumePreload();
					// ResumePreload may have found no further pages need to be preloaded, and it may have found some but immediately finished them and THEN found no further pages need to be preloaded.
					// So we can't assume bIsPreloading is still true after calling ResumePreload
					bLocalIsPreloading = bIsPreloading;
					continue;
				}
				else
				{
					// Turn off preloading for good, since we will be issuing synchronous IO to the same file from this point on
					bIsPreloading = false;
					bIsPreloadingPaused = false;
					bLocalIsPreloading = false;
				}
			}

			int64 ReadLength = EndPos - Pos;
			SerializeSynchronously(V, ReadLength);
			V = ((uint8*)V) + ReadLength;
			// SerializeBuffered incremented Pos
		}
	}
}

void FPreloadableArchive::SerializeSynchronously(void* V, int64 Length)
{
	if (SynchronousArchive)
	{
		SynchronousArchive->Seek(Pos);
		if (SynchronousArchive->IsError())
		{
			if (!IsError())
			{
				UE_LOGF(LogCore, Warning, "Failed to seek to offset %lld in %ls.", Pos, *GetArchiveName());
				SetError();
			}
		}
		else
		{
			SynchronousArchive->Serialize(V, Length);
			if (SynchronousArchive->IsError() && !IsError())
			{
				UE_LOGF(LogCore, Warning, "Failed to read %lld bytes at offset %lld in %ls.", Length, Pos, *GetArchiveName());
				SetError();
			}
		}
		Pos += Length;
	}
	else if (AsynchronousHandle)
	{
		if (Length == 0)
		{
			// 0 length ReadRequests are not allowed on IAsyncReadFileHandle
			return;
		}
		IAsyncReadRequest* ReadRequest = AsynchronousHandle->ReadRequest(Pos, Length, AIOP_Normal, nullptr, reinterpret_cast<uint8*>(V));
		if (!ReadRequest)
		{
			if (!IsError())
			{
				UE_LOGF(LogCore, Warning, "Failed to create ReadRequest to offset %lld in %ls.", Pos, *GetArchiveName());
				SetError();
			}
		}
		else
		{
			if (!ReadRequest->WaitCompletion())
			{
				if (!IsError())
				{
					UE_LOGF(LogCore, Warning, "Failed to WaitCompletion on ReadRequest to offset %lld in %ls.", Pos, *GetArchiveName());
					SetError();
				}
			}
			else
			{
				ensure(ReadRequest->GetReadResults() == V);
				Pos += Length;
			}

			delete ReadRequest;
		}
	}
	else
	{
		if (!IsError())
		{
			UE_LOGF(LogCore, Warning, "No InnerArchive available for serialization in %ls", *GetArchiveName());
			SetError();
		}
	}
	return;
}

void FPreloadableArchive::Seek(int64 InPos)
{
	checkf(InPos >= 0, TEXT("Attempted to seek to a negative location (%lld/%lld), file: %s. The file is most likely corrupt."), InPos, Size, *GetArchiveName());
	checkf(InPos <= Size, TEXT("Attempted to seek past the end of file (%lld/%lld), file: %s. The file is most likely corrupt."), InPos, Size, *GetArchiveName());
	Pos = InPos;
}

int64 FPreloadableArchive::Tell()
{
	return Pos;
}

int64 FPreloadableArchive::TotalSize()
{
	return Size;
}

bool FPreloadableArchive::Close()
{
	if (!bInitialized)
	{
		PendingAsyncComplete->Wait();
	}
	ReleaseCache();

	AsynchronousHandle.Reset();
	SynchronousArchive.Reset();
#if FPRELOADABLEFILE_TEST_ENABLED
	TestArchive.Reset();
#endif

	bInitialized = false;
	return !IsError();
}

FString FPreloadableArchive::GetArchiveName() const
{
	if (!ArchiveName.IsEmpty())
	{
		return ArchiveName;
	}
	else if (SynchronousArchive)
	{
		return SynchronousArchive->GetArchiveName();
	}
	else
	{
		return TEXT("FPreloadableArchive");
	}
}

TMap<FString, TSharedPtr<FPreloadableFile>> FPreloadableFile::RegisteredFiles;

FPreloadableFile::FPreloadableFile(FStringView FileName)
	: FPreloadableArchive(FileName)
{
	FPaths::MakeStandardFilename(ArchiveName);
}

void FPreloadableFile::InitializeAsync(uint32 InFlags, int64 PrimeSize)
{
	uint32 Mode = InFlags & Flags::ModeBits;
	switch (Mode)
	{
	case Flags::PreloadBytes:
		InitializeInternalAsync(
#if FPRELOADABLEFILE_TEST_ENABLED
			[this]() { return IFileManager::Get().CreateFileReader(*ArchiveName); },
#else
			FCreateArchive(),
#endif
			[this]() { return FPlatformFileManager::Get().GetPlatformFile().OpenAsyncRead(*ArchiveName); },
			InFlags, PrimeSize);
		break;
	case Flags::PreloadHandle:
		InitializeInternalAsync(
			[this]() { return IFileManager::Get().CreateFileReader(*ArchiveName); },
			FCreateAsyncArchive(),
			InFlags, PrimeSize);
		break;
	default:
		checkf(false, TEXT("Invalid mode %u."), Mode);
		break;
	}
}

bool FPreloadableFile::TryRegister(const TSharedPtr<FPreloadableFile>& PreloadableFile)
{
	if (!PreloadableFile || !PreloadableFile->IsInitialized() || PreloadableFile->TotalSize() < 0)
	{
		return false;
	}

	TSharedPtr<FPreloadableFile>& ExistingFile = RegisteredFiles.FindOrAdd(PreloadableFile->ArchiveName);
	if (ExistingFile)
	{
		return ExistingFile.Get() == PreloadableFile.Get();
	}

	ExistingFile = PreloadableFile;
	return true;
}

FArchive* FPreloadableFile::TryTakeArchive(const TCHAR* FileName)
{
	if (RegisteredFiles.Num() == 0)
	{
		return nullptr;
	}

	FString StandardFileName(FileName);
	FPaths::MakeStandardFilename(StandardFileName);
	TSharedPtr<FPreloadableFile> ExistingFile;
	if (!RegisteredFiles.RemoveAndCopyValue(*StandardFileName, ExistingFile))
	{
		return nullptr;
	}
	if (!ExistingFile->IsInitialized())
	{
		// Someone has called Close on the archive already.
		return nullptr;
	}
	FArchive* Result = ExistingFile->DetachLowerLevel();
	if (Result)
	{
		// the PreloadableArchive is in PreloadHandle mode; it is not preloading bytes, but instead is only providing a pre-opened (and possibly primed) sync handle
		return Result;
	}
	// Otherwise the archive is in PreloadBytes mode, and we need to return a proxy to it
	return new FPreloadableArchiveProxy(ExistingFile);
}

bool FPreloadableFile::UnRegister(const TSharedPtr<FPreloadableFile>& PreloadableFile)
{
	if (!PreloadableFile)
	{
		return false;
	}

	TSharedPtr<FPreloadableFile> ExistingFile;
	if (!RegisteredFiles.RemoveAndCopyValue(PreloadableFile->ArchiveName, ExistingFile))
	{
		return false;
	}

	if (ExistingFile.Get() != PreloadableFile.Get())
	{
		// Some other FPreloadableFile was registered for the same FileName. We removed it in the RemoveAndCopyValue above (which we do to optimize the common case).
		// Add it back, and notify the caller that their PreloadableFile was not registered.
		RegisteredFiles.Add(PreloadableFile->ArchiveName, MoveTemp(ExistingFile));
		return false;
	}

	return true;
}
