// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/QueuedThreadPoolWrapper.h"

#include "HAL/PlatformProcess.h"
#include "Misc/IQueuedWork.h"
#include "Misc/QueuedThreadPool.h"
#include "ProfilingDebugging/CountersTrace.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "Templates/Tuple.h"
#include "Templates/UnrealTemplate.h"

TRACE_DECLARE_INT_COUNTER(QueuedThreadPoolWrapperScheduledWorkAllocs, TEXT("QueuedThreadPoolWrapper/ScheduledWorkAllocs"));

namespace QueuedThreadPoolWrapperImpl
{
	static std::atomic<int64> ScheduledWorkAllocs {0};
}

FQueuedThreadPoolWrapper::FScheduledWork::FScheduledWork()
{
	TRACE_COUNTER_SET(QueuedThreadPoolWrapperScheduledWorkAllocs, ++QueuedThreadPoolWrapperImpl::ScheduledWorkAllocs);
}

FQueuedThreadPoolWrapper::FScheduledWork::~FScheduledWork()
{
	check(NumRefs.GetValue() == 0);
	TRACE_COUNTER_SET(QueuedThreadPoolWrapperScheduledWorkAllocs, --QueuedThreadPoolWrapperImpl::ScheduledWorkAllocs);
}

FQueuedThreadPoolWrapper::FQueuedThreadPoolWrapper(FQueuedThreadPool* InWrappedQueuedThreadPool, int32 InMaxConcurrency, TFunction<EQueuedWorkPriority(EQueuedWorkPriority)> InPriorityMapper)
	: PriorityMapper(InPriorityMapper)
	, WrappedQueuedThreadPool(InWrappedQueuedThreadPool)
	, MaxConcurrency(InMaxConcurrency == -1 ? InWrappedQueuedThreadPool->GetNumThreads() : InMaxConcurrency)
	, MaxTaskToSchedule(-1)
	, CurrentConcurrency(0)
{
}

FQueuedThreadPoolWrapper::~FQueuedThreadPoolWrapper()
{
	Destroy();
}

bool FQueuedThreadPoolWrapper::Create(uint32 InNumQueuedThreads, uint32 StackSize, EThreadPriority ThreadPriority, const TCHAR* Name)
{
	return true;
}

void FQueuedThreadPoolWrapper::Destroy()
{
	{
		FScopeLock ScopeLock(&Lock);

		// Clean up all queued objects
		while (IQueuedWork* WorkItem = QueuedWork.Dequeue())
		{
			WorkItem->Abandon();
		}

		QueuedWork.Reset();

		// Try to retract anything already in flight
		TArray<FScheduledWork*> ScheduledWorkToRetract;
		ScheduledWork.GenerateValueArray(ScheduledWorkToRetract);

		for (FScheduledWork* Work : ScheduledWorkToRetract)
		{
			if (WrappedQueuedThreadPool->RetractQueuedWork(Work))
			{
				Work->GetInnerWork()->Abandon();
				// Decrement the ref added by Assign() before releasing.
				// Matches the pattern in RetractQueuedWork() and TryRetractWorkNoLock().
				check(Work->NumRefs.GetValue() == 1);
				Work->NumRefs.Decrement();
				ReleaseWorkNoLock(Work);
			}
		}
	}

	if (CurrentConcurrency)
	{
		// We can't delete our WorkPool elements
		// if they're still referenced by a threadpool.
		// Retraction didn't work, so no choice to wait until they're all finished.
		TRACE_CPUPROFILER_EVENT_SCOPE(FQueuedThreadPoolWrapper::DestroyWait);
		while (CurrentConcurrency)
		{
			FPlatformProcess::Sleep(0.1f);
		}
	}

	{
		FScopeLock ScopeLock(&Lock);
		for (FScheduledWork* Work : WorkPool)
		{
			check(Work->GetInnerWork() == nullptr);
			delete Work;
		}
		WorkPool.Empty();
	}
}

void FQueuedThreadPoolWrapper::SetMaxConcurrency(int32 InMaxConcurrency)
{
	MaxConcurrency = InMaxConcurrency == -1 ? WrappedQueuedThreadPool->GetNumThreads() : InMaxConcurrency;

	// We might need to schedule or unshedule tasks depending on how MaxConcurrency has changed.
	Schedule();
}

void FQueuedThreadPoolWrapper::Pause()
{
	FScopeLock ScopeLock(&Lock);
	MaxTaskToSchedule = 0;
}

void FQueuedThreadPoolWrapper::Resume(int32 InNumQueuedWork)
{
	{
		FScopeLock ScopeLock(&Lock);
		MaxTaskToSchedule = InNumQueuedWork;
	}

	Schedule();
}

void FQueuedThreadPoolWrapper::AddQueuedWork(IQueuedWork* InQueuedWork, EQueuedWorkPriority InPriority)
{
	{
		FScopeLock ScopeLock(&Lock);
		QueuedWork.Enqueue(InQueuedWork, InPriority);
	}

	Schedule();
}

bool FQueuedThreadPoolWrapper::RetractQueuedWork(IQueuedWork* InQueuedWork)
{
	FScheduledWork* Retracted = nullptr;
	{
		FScopeLock ScopeLock(&Lock);
		if (QueuedWork.Retract(InQueuedWork))
		{
			return true;
		}
		
		Retracted = ScheduledWork.FindRef(InQueuedWork);
		if (Retracted && !WrappedQueuedThreadPool->RetractQueuedWork(Retracted))
		{
			Retracted = nullptr;
		}
	}

	if (Retracted)
	{
		// When we retract a task, there should be no external refs on it
		check(Retracted->NumRefs.GetValue() == 1);
		Retracted->Release();
	}
	return Retracted != nullptr;
}

int32 FQueuedThreadPoolWrapper::GetNumThreads() const
{
	return MaxConcurrency;
}

void FQueuedThreadPoolWrapper::ReleaseWorkNoLock(FScheduledWork* Work)
{
	check(Work->NumRefs.GetValue() == 0);
	CurrentConcurrency--;
	OnUnscheduled(Work);
	
	ScheduledWork.Remove(Work->GetInnerWork());
	Work->Reset();
	check(Work->NumRefs.GetValue() == 0);
	WorkPool.Push(Work);
}

bool FQueuedThreadPoolWrapper::CanSchedule(EQueuedWorkPriority Priority) const
{
	return (MaxTaskToSchedule == -1 || MaxTaskToSchedule > 0 || Priority == EQueuedWorkPriority::Blocking) && CurrentConcurrency < GetMaxConcurrency();
}

FQueuedThreadPoolWrapper::FScheduledWork* FQueuedThreadPoolWrapper::AllocateWork(IQueuedWork* InnerWork, EQueuedWorkPriority Priority)
{
	FScheduledWork* Work = nullptr;
	if (WorkPool.Num() > 0)
	{
		Work = WorkPool.Pop(EAllowShrinking::No);
	}
	else
	{
		Work = AllocateScheduledWork();
	}

	Work->Assign(this, InnerWork, Priority);
	return Work;
}

bool FQueuedThreadPoolWrapper::TryRetractWorkNoLock(EQueuedWorkPriority InPriority)
{
	// Scheduled work is bound by MaxConcurrency which is normally limited by Core count. 
	// The linear scan should be small and pretty fast.
	for (TTuple<IQueuedWork*, FScheduledWork*>& Pair : ScheduledWork)
	{
		// higher number means lower priority
		if (Pair.Value->GetPriority() > InPriority)
		{
			if (WrappedQueuedThreadPool->RetractQueuedWork(Pair.Value))
			{
				// When we retract a task, there should be no external refs on it
				check(Pair.Value->NumRefs.GetValue() == 1);
				QueuedWork.Enqueue(Pair.Key, Pair.Value->GetPriority());
				Pair.Value->Release();

				if (MaxTaskToSchedule != -1)
				{
					MaxTaskToSchedule++;
				}

				return true;
			}
		}
	}

	return false;
}

void FQueuedThreadPoolWrapper::Schedule(FScheduledWork* Work)
{
	FScopeLock ScopeLock(&Lock);

	// It's important to reduce the current concurrency before entering the loop
	// especially when MaxConcurrency is 1 to ensure new work is scheduled and don't 
	// end up never being called again.
	if (Work != nullptr)
	{
		ReleaseWorkNoLock(Work);
	}

	// In case we call Release on lower priority tasks we retract as part of the scheduling loop,
	// we might end up calling Schedule recursively. We avoid reentrancy on the scheduling loop once the work item has
	// been released in ReleaseWorkNoLock by early exiting once we reach the loop part as there is nothing more we can do.
	if (bIsScheduling)
	{
		return;
	}

	// We are now entering the scheduling loop
	TGuardValue<bool> SchedulingScope(bIsScheduling, true);

	// If a higher priority task comes in, try to retract lower priority ones if possible to make room
	EQueuedWorkPriority NextWorkPriority = EQueuedWorkPriority::Count;
	IQueuedWork* NextWork = QueuedWork.Peek(&NextWorkPriority);
		 
	// When NextWork is nullptr here, it means that QueuedWork.Peek did not 
	// find any work at any priority. That means NextWorkPriority is not 
	// valid, there is no work to try schedule, so early out.
	if (NextWork == nullptr)
	{
		return;
	}

	// Continue retracting more work until nothing can be retracted anymore or we can finally squeeze the higher priority task in
	while (!CanSchedule(NextWorkPriority) && TryRetractWorkNoLock(NextWorkPriority))
	{
	}

	// Schedule as many tasks we can fit
	// Note: The reason for checking that there is work queued is that CanSchedule calls GetMaxConcurrency 
	// 	which is virtual and can (and does at times) do work that is only required if there is more work to 
	// 	schedule. For example, adjusting max concurrency based on currently available memory.
	while (QueuedWork.Num() > 0 && CanSchedule(NextWorkPriority))
	{
		EQueuedWorkPriority WorkPriority;
		IQueuedWork* InnerWork = QueuedWork.Dequeue(&WorkPriority);

		if (InnerWork)
		{
			CurrentConcurrency++;
			
			Work = AllocateWork(InnerWork, WorkPriority);
			ScheduledWork.Add(InnerWork, Work);
			OnScheduled(Work);
			WrappedQueuedThreadPool->AddQueuedWork(Work, WorkPriority == EQueuedWorkPriority::Blocking ? WorkPriority : PriorityMapper(WorkPriority));

			if (MaxTaskToSchedule > 0)
			{
				MaxTaskToSchedule--;
			}
		}
		else
		{
			break;
		}
	}
}