// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/RemoteConfigIni.h"

#include "Async/AsyncWork.h"
#include "CoreGlobals.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformProcess.h"
#include "HAL/PlatformTime.h"
#include "HAL/UnrealMemory.h"
#include "Misc/App.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CString.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Misc/ScopeLock.h"
#include "Stats/Stats.h"
#include "Templates/TypeHash.h"

// Globals
FRemoteConfig GRemoteConfig;
FRemoteConfigAsyncTaskManager GRemoteConfigIOManager;


/*-----------------------------------------------------------------------------
   FRemoteConfigAsyncIOInfo
-----------------------------------------------------------------------------*/

FRemoteConfigAsyncIOInfo::FRemoteConfigAsyncIOInfo(const TCHAR* InDefaultIniFile)
	: bReadIOFailed(false)
	, bWasProcessed(false)
{
	FCString::Strcpy(DefaultIniFile, InDefaultIniFile);
}


/*-----------------------------------------------------------------------------
   FRemoteConfigAsyncWorker
-----------------------------------------------------------------------------*/

FRemoteConfigAsyncWorker::FRemoteConfigAsyncWorker(const TCHAR* InFilename, FRemoteConfigAsyncIOInfo& InIOInfo, FString* InContents, bool bInIsRead)
{
	check(FCString::Strlen(InFilename) < 1024);
	FCString::Strcpy(Filename, InFilename);
	bIsRead = bInIsRead;
	IOInfo = InIOInfo;
	if (InContents)
	{
		// Only used in write operations
		Contents = *InContents;
	}
}


void FRemoteConfigAsyncWorker::DoWork()
{
	if (bIsRead)
	{
		// Do read
		IOInfo.TimeStamp = IFileManager::Get().GetTimeStamp(Filename);
		IOInfo.bReadIOFailed = !FFileHelper::LoadFileToString(IOInfo.Buffer, Filename);
	} 
	else
	{
		// Do write
		if (!Contents.IsEmpty())
		{
			FFileHelper::SaveStringToFile(Contents, Filename);
		}
	}
}


bool FRemoteConfigAsyncWorker::IsReadSuccess() const
{
	return !IOInfo.bReadIOFailed;
}


FRemoteConfigAsyncIOInfo& FRemoteConfigAsyncWorker::GetIOInfo()
{
	return IOInfo;
}



bool FRemoteConfigAsyncWorker::CanAbandon()
{
	//return !bIsRead;
	return false;
}


void FRemoteConfigAsyncWorker::Abandon()
{
	// @todo what goes in here?
}


/*-----------------------------------------------------------------------------
   FRemoteConfigAsyncTaskManager
-----------------------------------------------------------------------------*/

FRemoteConfigAsyncTaskManager* FRemoteConfigAsyncTaskManager::Get()
{
	return &GRemoteConfigIOManager;
}


void FRemoteConfigAsyncTaskManager::Tick()
{
	FScopeLock ScopeLock(&SynchronizationObject);
	
	for ( auto It = CachedWriteTasks.CreateIterator(); It; ++It)
	{
		if (GRemoteConfig.Write(*It->Filename, It->Contents))
		{
			It.RemoveCurrent();
		}
	}
}


bool FRemoteConfigAsyncTaskManager::FindCachedWriteTask(const TCHAR* InFilename, bool bCompareContents, const TCHAR* InContents)
{
	for (int32 Idx = 0; Idx < CachedWriteTasks.Num(); ++Idx)
	{
		if (!FCString::Stricmp(InFilename, *CachedWriteTasks[Idx].Filename) &&
			(!bCompareContents || !FCString::Stricmp(InContents, *CachedWriteTasks[Idx].Contents)))
		{
			return true;
		}
	}
	return false;
}


bool FRemoteConfigAsyncTaskManager::StartTask(const TCHAR* InFilename, const TCHAR* RemotePath, FRemoteConfigAsyncIOInfo& InIOInfo, FString* InContents, bool bInIsRead)
{
	FScopeLock ScopeLock(&SynchronizationObject);
	FAsyncTask<FRemoteConfigAsyncWorker>* AsyncTask = PendingTasks.FindRef(FString(InFilename));

	// See if a task for this file already exists
	if (AsyncTask)
	{
		if (!bInIsRead)
		{
			if (AsyncTask->IsDone())
			{
				// Clear out old write tasks that have completed
				delete AsyncTask;
				PendingTasks.Remove(FString(InFilename));
			}
			else
			{
				// Handle cached write tasks
				if (!FindCachedWriteTask(InFilename, true, **InContents))
				{
					CachedWriteTasks.Add(FRemoteConfigAsyncCachedWriteTask(InFilename, InContents));
				}
				return false;
			}
		}
		else 
		{
			// Early out if there still a read task in the queue
			return false;
		}
	}

	// Add new task to the queue and start it
	FAsyncTask<FRemoteConfigAsyncWorker>*& NewTask = PendingTasks.Add(FString(InFilename), new FAsyncTask<FRemoteConfigAsyncWorker>(RemotePath, InIOInfo, InContents, bInIsRead));
	NewTask->StartBackgroundTask();
	
	return true;
}


bool FRemoteConfigAsyncTaskManager::IsFinished(const TCHAR* InFilename)
{
	FScopeLock ScopeLock(&SynchronizationObject);
	FAsyncTask<FRemoteConfigAsyncWorker>* AsyncTask = NULL;
	AsyncTask = PendingTasks.FindRef(FString(InFilename));
		
	return AsyncTask? AsyncTask->IsDone(): true;
}


bool FRemoteConfigAsyncTaskManager::AreAllTasksFinished(bool bDoRemoval)
{
	FScopeLock ScopeLock(&SynchronizationObject);

	for (TMap<FString, FAsyncTask<FRemoteConfigAsyncWorker>* >::TIterator It(PendingTasks); It; ++It)
	{
		// If any cached write tasks exist, tasks are clearly not finished
		if (CachedWriteTasks.Num() > 0)
		{
			return false;
		}

		FAsyncTask<FRemoteConfigAsyncWorker>* AsyncTask = NULL;
		AsyncTask = PendingTasks.FindRef(It.Key());
			
		if (AsyncTask)
		{
			if (AsyncTask->IsDone() && bDoRemoval)
			{
				// Remove completed tasks from the queue if so desired
				delete AsyncTask;
				It.RemoveCurrent();
			}
			else
			{
				// Return on the first unfinished test
				return false;
			}
		}
	}
		
	return true;
}


bool FRemoteConfigAsyncTaskManager::GetReadData(const TCHAR* InFilename, FRemoteConfigAsyncIOInfo& OutIOInfo)
{
	FScopeLock ScopeLock(&SynchronizationObject);
	FAsyncTask<FRemoteConfigAsyncWorker>* AsyncTask = NULL;
		
	// Early out if the task hasn't finished
	if (!IsFinished(InFilename))
	{
		return false;
	}

	PendingTasks.RemoveAndCopyValue(FString(InFilename), AsyncTask);
	check(AsyncTask);
		
	bool RetVal = AsyncTask->GetTask().IsReadSuccess();
	OutIOInfo = AsyncTask->GetTask().GetIOInfo();
	delete AsyncTask;
	return RetVal;
}


/*-----------------------------------------------------------------------------
   FRemoteConfig
-----------------------------------------------------------------------------*/

FRemoteConfig::FRemoteConfig()
	: Timeout(-1.0f)
	, bIsEnabled(true)
	, bHasCachedFilenames(false)
{ }


FRemoteConfig* FRemoteConfig::Get()
{
	return &GRemoteConfig;
}


bool FRemoteConfig::IsRemoteFile(const TCHAR* Filename)
{
	FString IniFileName(Filename);
	FString BaseFilename = FPaths::GetBaseFilename(IniFileName);

	if (!bHasCachedFilenames && GConfig->FindConfigFile(GEngineIni))
	{
		// Read in the list of desired remote files once and only once
		GConfig->GetArray(TEXT("RemoteConfiguration"), TEXT("IniToLoad"), CachedFileNames, GEngineIni);
		bHasCachedFilenames = true;

		// Cache off the "enabled" flag while we're at it
		GConfig->GetBool(TEXT("RemoteConfiguration"), TEXT("Enabled"), bIsEnabled, GEngineIni);

	}

	// Return false if remote config functionality has been flagged as disabled
	if (!bIsEnabled)
	{
		return false;
	}

	// See if the specified file exists in the list of desired remote files
	for (int32 Idx = 0; Idx < CachedFileNames.Num(); ++Idx)
	{
		if (!FCString::Stricmp(*BaseFilename, *CachedFileNames[Idx]))
		{
			return true;
		}
	}

	return false;
}


bool FRemoteConfig::ShouldReadRemoteFile(const TCHAR* Filename)
{
	return IsRemoteFile(Filename) && !FindConfig(Filename);
}


FRemoteConfigAsyncIOInfo* FRemoteConfig::FindConfig(const TCHAR* Filename)
{
	return ConfigBuffers.FindByHash(FCrc::Strihash_DEPRECATED(Filename), Filename);
}


bool FRemoteConfig::IsFinished(const TCHAR* InFilename)
{
	return GRemoteConfigIOManager.IsFinished(InFilename);
}
	

bool FRemoteConfig::Read(const TCHAR* GeneratedIniFile, const TCHAR* DefaultIniFile)
{
	FString FullPath = GenerateRemotePath(GeneratedIniFile);

	if (Timeout < 0.0f)
	{
		// Store off the async IO timeout
		GConfig->GetFloat(TEXT("RemoteConfiguration"), TEXT("Timeout"), Timeout, GEngineIni);
	}

	FRemoteConfigAsyncIOInfo& IOInfo = ConfigBuffers.Add(FString(GeneratedIniFile), FRemoteConfigAsyncIOInfo(DefaultIniFile));
		
	IOInfo.StartReadTime = FPlatformTime::Seconds();
	return GRemoteConfigIOManager.StartTask(GeneratedIniFile, *FullPath, IOInfo, NULL, true);
}


bool FRemoteConfig::Write(const TCHAR* Filename, FString& Contents)
{
	FRemoteConfigAsyncIOInfo* IOInfo = FindConfig(Filename);
	// Assuming here that if we remotely loaded an config file, we'll want to remotely save it too
	if (IOInfo)
	{
		FString FullPath = GenerateRemotePath(Filename);

		IOInfo->StartWriteTime = FPlatformTime::Seconds();
		return GRemoteConfigIOManager.StartTask(Filename, *FullPath, *IOInfo, &Contents, false);
	}

	return true;
}


void FRemoteConfig::FinishRead(const TCHAR* Filename)
{
	FRemoteConfigAsyncIOInfo* IOInfo = FindConfig(Filename);
	if (IOInfo && !IOInfo->bWasProcessed)
	{
		while (!GRemoteConfigIOManager.IsFinished(Filename))
		{
			if ((FPlatformTime::Seconds() - IOInfo->StartReadTime) > Timeout)
			{
				// Time out
				IOInfo->bReadIOFailed = true;
				break;
			}
		}

		// Now, process the config file
		FString DestFileName(Filename);
		GRemoteConfigIOManager.GetReadData(Filename, *IOInfo);
		IOInfo->bWasProcessed = true;
		FConfigCacheIni::LoadGlobalIniFile(DestFileName, IOInfo->DefaultIniFile);
	}
}


void FRemoteConfig::Flush()
{
	// @todo this is hacky... figure out how to clean up properly
	double Time = FPlatformTime::Seconds();
	while (!GRemoteConfigIOManager.AreAllTasksFinished(true)) 
	{
		GRemoteConfigIOManager.Tick();
		if ((FPlatformTime::Seconds() - Time) > GRemoteConfig.Timeout)
		{
			break;
		}
	}
}

// Special chars derived from ParseLineExtended()
#define NUM_SPECIAL_CHARS 6
static const TCHAR* SpecialCharMap[NUM_SPECIAL_CHARS][2] =
{
	// ORDER IS IMPORTANT, we must go from the most generic/global to specific, the reason being that a key could be named the same as the one listed here and since they
	// are in quote it will break the assumption done here.
	{ TEXT("{"), TEXT("~OpenBracket~") },
	{ TEXT("}"), TEXT("~CloseBracket~") },
	{ TEXT("\""), TEXT("~Quote~") },
	{ TEXT("\\"), TEXT("~Backslash~") },
	{ TEXT("/"), TEXT("~Forwardslash~") },
	{ TEXT("|"), TEXT("~Bar~") }
};


FString FRemoteConfig::ReplaceIniCharWithSpecialChar(const FString& Str)
{
	FString Result = Str;
	for (int32 Idx = 0; Idx < NUM_SPECIAL_CHARS; ++Idx)
	{
		Result = Result.Replace(SpecialCharMap[Idx][0], SpecialCharMap[Idx][1]);
	}
	return Result;
}


FString FRemoteConfig::ReplaceIniSpecialCharWithChar(const FString& Str)
{
	FString Result = Str;
	for (int32 Idx = 0; Idx < NUM_SPECIAL_CHARS; ++Idx)
	{
		Result = Result.Replace(SpecialCharMap[Idx][1], SpecialCharMap[Idx][0]);
	}
	return Result;
}


FString FRemoteConfig::GenerateRemotePath(const TCHAR* Filename)
{
	FString IniFileName(Filename);
	FString BaseFilename = FPaths::GetBaseFilename(IniFileName);
	FString PathPrefix = GConfig->GetStr(TEXT("RemoteConfiguration"), TEXT("ConfigPathPrefix"), GEngineIni);
	FString PathSuffix = GConfig->GetStr(TEXT("RemoteConfiguration"), TEXT("ConfigPathSuffix"), GEngineIni);
	FString UserName = FPlatformProcess::UserName(false);
		
	return FString::Printf(TEXT("%s/%s/%s/%s/%s.ini"), *PathPrefix, *UserName, *PathSuffix, FApp::GetProjectName(), *BaseFilename);
}


/*-----------------------------------------------------------------------------
   Helper/utility methods
-----------------------------------------------------------------------------*/

/** Returns true if no remote version of this config file exists and/or isn't being used */
bool IsUsingLocalIniFile(const TCHAR* FilenameToLoad, const TCHAR* IniFileName)
{
	check(FilenameToLoad);
	FRemoteConfigAsyncIOInfo* RemoteInfo = GRemoteConfig.FindConfig(FilenameToLoad);
	bool bIsGeneratedFile = IniFileName? !FCString::Stricmp(FilenameToLoad, IniFileName): true;
	return !RemoteInfo || !bIsGeneratedFile || (RemoteInfo && RemoteInfo->bReadIOFailed);
}

/** Contains the logic for processing config files, local or remote */
void ProcessIniContents(const TCHAR* FilenameToLoad, const TCHAR* IniFileName, FConfigFile* Config, bool bDoEmptyConfig, bool bDoCombine)
{
	DECLARE_SCOPE_CYCLE_COUNTER( TEXT( "ProcessIniContents" ), STAT_ProcessIniContents, STATGROUP_LoadTime );

	check(FilenameToLoad);
	check(IniFileName);
	check(Config);
	if (IsUsingLocalIniFile(FilenameToLoad, IniFileName))
	{
		// Use local file

		if (bDoCombine)
		{
			Config->Combine(IniFileName);
		}
		else
		{
			Config->Read(IniFileName);
		}
	}
	else
	{
		// Use remote file

		FRemoteConfigAsyncIOInfo* RemoteInfo = GRemoteConfig.FindConfig(FilenameToLoad);
		
		if (bDoEmptyConfig) 
		{
			Config->Empty();
		}
		
		if (bDoCombine)
		{
			Config->CombineFromBuffer(RemoteInfo->Buffer, FilenameToLoad);
		}
		else
		{
			Config->ProcessInputFileContents(RemoteInfo->Buffer, FilenameToLoad);
		}
	}
}

/** Returns the timestamp of the appropriate config file */
FDateTime GetIniTimeStamp(const TCHAR* FilenameToLoad, const TCHAR* IniFileName)
{
	check(FilenameToLoad);
	check(IniFileName);
	if (IsUsingLocalIniFile(FilenameToLoad, IniFileName))
	{
		return IFileManager::Get().GetTimeStamp(IniFileName);
	}
	else
	{
		FRemoteConfigAsyncIOInfo* RemoteInfo = GRemoteConfig.FindConfig(FilenameToLoad);
		return RemoteInfo->TimeStamp;
	}
}

/** Before overwriting the local file with the contents from the remote file, we save off a copy of the local file (if it exists) */
void MakeLocalCopy(const TCHAR* Filename)
{
	check(Filename);
	if (IsUsingLocalIniFile(Filename, NULL))
	{
		// Remote file isn't being used, no need to make a copy of the local file
		return;
	}

	if (IFileManager::Get().FileSize(Filename) >= 0)
	{
		FString FilenameStr = Filename;
		if (FCString::Stristr(*FilenameStr, TEXT(".ini")))
		{
			FilenameStr.LeftChopInline(4, EAllowShrinking::No);
		} 
		else
		{
			check(false);
		}
	
		TCHAR FilenameLocal[1024];
		FCString::Strcpy(FilenameLocal, *FilenameStr);
		FCString::Strcat(FilenameLocal, TEXT("_Local.ini"));
		IFileManager::Get().Copy(FilenameLocal, Filename);
	}
}
