// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/RuntimeErrors.h"
#include "Misc/AssertionMacros.h"
#include "Internationalization/Text.h"

FRuntimeErrors::FRuntimeErrorDelegate FRuntimeErrors::OnRuntimeIssueLogged;

void FRuntimeErrors::LogRuntimeIssue(ELogVerbosity::Type Verbosity, const ANSICHAR* FileName, int32 LineNumber, const FText& Message)
{
	OnRuntimeIssueLogged.Execute(Verbosity, FileName, LineNumber, Message);
}

bool FRuntimeErrors::LogRuntimeIssueReturningFalse(const ANSICHAR* Expr, const ANSICHAR* File, int32 Line)
{
#if UE_RAISE_RUNTIME_ERRORS
	// Print initial debug message for this error
	TCHAR ErrorString[MAX_SPRINTF];
	FCString::Sprintf(ErrorString, TEXT("ensureAsRuntimeWarning condition failed: %s"), ANSI_TO_TCHAR(Expr));

	FRuntimeErrors::LogRuntimeIssue(ELogVerbosity::Error, File, Line, FText::AsCultureInvariant(FString(ErrorString)));
#endif
	return false;
}
