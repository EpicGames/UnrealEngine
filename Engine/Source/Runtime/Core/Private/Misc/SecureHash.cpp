// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/SecureHash.h"

#include "Async/AsyncWork.h"
#include "HAL/FileManager.h"
#include "Memory/MemoryView.h"
#include "Misc/Paths.h"
#include "Misc/StringBuilder.h"
#include "Serialization/CompactBinaryWriter.h"

#if (defined(_M_AMD64) || defined(__x86_64__)) && !defined(_M_ARM64EC)

#include <immintrin.h>
#if PLATFORM_COMPILER_CLANG
#include <cpuid.h>
#include <shaintrin.h>
#endif

#define UE_PLATFORM_SHA_X86 1

// if x86 SHA is enabled with compiler, use SHA instructions without fallback
#if defined(__SHA__) || PLATFORM_ALWAYS_HAS_SHA
#define UE_PLATFORM_SHA_FALLBACK 0
#endif

#elif defined(__aarch64__) || defined(_M_ARM64) || defined(_M_ARM64EC)

// on ARMv8 enable SHA instructions unconditionally only when they are enabled with compiler
#ifdef __ARM_FEATURE_CRYPTO
#define UE_PLATFORM_SHA_ARMV8 1
#define UE_PLATFORM_SHA_FALLBACK 0
#endif

#endif


#ifndef UE_PLATFORM_SHA_FALLBACK
#define UE_PLATFORM_SHA_FALLBACK 1
#endif
#ifndef UE_PLATFORM_SHA_X86
#define UE_PLATFORM_SHA_X86 0
#endif
#ifndef UE_PLATFORM_SHA_ARMV8
#define UE_PLATFORM_SHA_ARMV8 0
#endif


DEFINE_LOG_CATEGORY_STATIC(LogSecureHash, Log, All);


/*-----------------------------------------------------------------------------
	MD5 functions, adapted from MD5 RFC by Brandon Reinhart
-----------------------------------------------------------------------------*/

//
// Constants for MD5 Transform.
//

enum {_S11=7};
enum {_S12=12};
enum {_S13=17};
enum {_S14=22};
enum {_S21=5};
enum {_S22=9};
enum {_S23=14};
enum {_S24=20};
enum {_S31=4};
enum {_S32=11};
enum {_S33=16};
enum {_S34=23};
enum {_S41=6};
enum {_S42=10};
enum {_S43=15};
enum {_S44=21};

static const uint8 PADDING[64] = { 0x80 };

//
// Basic MD5 transformations.
//
#define MD5_F(x, y, z) (((x) & (y)) | ((~x) & (z)))
#define MD5_G(x, y, z) (((x) & (z)) | ((y) & (~z)))
#define MD5_H(x, y, z) ((x) ^ (y) ^ (z))
#define MD5_I(x, y, z) ((y) ^ ((x) | (~z)))

//
// Rotates X left N bits.
//
#define ROTLEFT(x, n) (((x) << (n)) | ((x) >> (32-(n))))

//
// Rounds 1, 2, 3, and 4 MD5 transformations.
// Rotation is separate from addition to prevent recomputation.
//
#define MD5_FF(a, b, c, d, x, s, ac) { \
	(a) += MD5_F ((b), (c), (d)) + (x) + (uint32)(ac); \
	(a) = ROTLEFT ((a), (s)); \
	(a) += (b); \
}

#define MD5_GG(a, b, c, d, x, s, ac) { \
	(a) += MD5_G ((b), (c), (d)) + (x) + (uint32)(ac); \
	(a) = ROTLEFT ((a), (s)); \
	(a) += (b); \
}

#define MD5_HH(a, b, c, d, x, s, ac) { \
	(a) += MD5_H ((b), (c), (d)) + (x) + (uint32)(ac); \
	(a) = ROTLEFT ((a), (s)); \
	(a) += (b); \
}

#define MD5_II(a, b, c, d, x, s, ac) { \
	(a) += MD5_I ((b), (c), (d)) + (x) + (uint32)(ac); \
	(a) = ROTLEFT ((a), (s)); \
	(a) += (b); \
}

FMD5::FMD5()
{
	Context.count[0] = Context.count[1] = 0;
	// Load magic initialization constants.
	Context.state[0] = 0x67452301;
	Context.state[1] = 0xefcdab89;
	Context.state[2] = 0x98badcfe;
	Context.state[3] = 0x10325476;
}

FMD5::~FMD5()
{

}

void FMD5::Update( const uint8* input, uint64 inputLen )
{
	uint64 i, index, partLen;

	// Compute number of bytes mod 64.
	index = (int32)((Context.count[0] >> 3) & 0x3F);

	// Update number of bits.
	uint64 inputBitCount = inputLen << 3;
	uint64 newBitCount = (uint64(Context.count[0]) | (uint64(Context.count[1]) << 32)) + inputBitCount;
	Context.count[0] = uint32(newBitCount);
	Context.count[1] = uint32(newBitCount >> 32);

	partLen = 64 - index;

	// Transform as many times as possible.
	if (inputLen >= partLen) 
	{
		FMemory::Memcpy( &Context.buffer[index], input, partLen );
		Transform( Context.state, Context.buffer );
		for (i = partLen; i + 63 < inputLen; i += 64)
		{
			Transform( Context.state, &input[i] );
		}
		index = 0;
	} 
	else
	{
		i = 0;
	}

	// Buffer remaining input.
	FMemory::Memcpy( &Context.buffer[index], &input[i], inputLen-i );
}

void FMD5::Final( uint8* digest )
{
	uint8 bits[8];
	int32 index, padLen;

	// Save number of bits.
	Encode( bits, Context.count, 8 );

	// Pad out to 56 mod 64.
	index = (int32)((Context.count[0] >> 3) & 0x3f);
	padLen = (index < 56) ? (56 - index) : (120 - index);
	Update( PADDING, padLen );

	// Append length (before padding).
	Update( bits, 8 );

	// Store state in digest
	Encode( digest, Context.state, 16 );

	// Zeroize sensitive information.
	FMemory::Memset( &Context, 0, sizeof(Context) );
}

void FMD5::Transform( uint32* state, const uint8* block )
{
	uint32 a = state[0], b = state[1], c = state[2], d = state[3], x[16];

	Decode( x, block, 64 );

	// Round 1
	MD5_FF (a, b, c, d, x[ 0], _S11, 0xd76aa478); /* 1 */
	MD5_FF (d, a, b, c, x[ 1], _S12, 0xe8c7b756); /* 2 */
	MD5_FF (c, d, a, b, x[ 2], _S13, 0x242070db); /* 3 */
	MD5_FF (b, c, d, a, x[ 3], _S14, 0xc1bdceee); /* 4 */
	MD5_FF (a, b, c, d, x[ 4], _S11, 0xf57c0faf); /* 5 */
	MD5_FF (d, a, b, c, x[ 5], _S12, 0x4787c62a); /* 6 */
	MD5_FF (c, d, a, b, x[ 6], _S13, 0xa8304613); /* 7 */
	MD5_FF (b, c, d, a, x[ 7], _S14, 0xfd469501); /* 8 */
	MD5_FF (a, b, c, d, x[ 8], _S11, 0x698098d8); /* 9 */
	MD5_FF (d, a, b, c, x[ 9], _S12, 0x8b44f7af); /* 10 */
	MD5_FF (c, d, a, b, x[10], _S13, 0xffff5bb1); /* 11 */
	MD5_FF (b, c, d, a, x[11], _S14, 0x895cd7be); /* 12 */
	MD5_FF (a, b, c, d, x[12], _S11, 0x6b901122); /* 13 */
	MD5_FF (d, a, b, c, x[13], _S12, 0xfd987193); /* 14 */
	MD5_FF (c, d, a, b, x[14], _S13, 0xa679438e); /* 15 */
	MD5_FF (b, c, d, a, x[15], _S14, 0x49b40821); /* 16 */

	// Round 2
	MD5_GG (a, b, c, d, x[ 1], _S21, 0xf61e2562); /* 17 */
	MD5_GG (d, a, b, c, x[ 6], _S22, 0xc040b340); /* 18 */
	MD5_GG (c, d, a, b, x[11], _S23, 0x265e5a51); /* 19 */
	MD5_GG (b, c, d, a, x[ 0], _S24, 0xe9b6c7aa); /* 20 */
	MD5_GG (a, b, c, d, x[ 5], _S21, 0xd62f105d); /* 21 */
	MD5_GG (d, a, b, c, x[10], _S22,  0x2441453); /* 22 */
	MD5_GG (c, d, a, b, x[15], _S23, 0xd8a1e681); /* 23 */
	MD5_GG (b, c, d, a, x[ 4], _S24, 0xe7d3fbc8); /* 24 */
	MD5_GG (a, b, c, d, x[ 9], _S21, 0x21e1cde6); /* 25 */
	MD5_GG (d, a, b, c, x[14], _S22, 0xc33707d6); /* 26 */
	MD5_GG (c, d, a, b, x[ 3], _S23, 0xf4d50d87); /* 27 */
	MD5_GG (b, c, d, a, x[ 8], _S24, 0x455a14ed); /* 28 */
	MD5_GG (a, b, c, d, x[13], _S21, 0xa9e3e905); /* 29 */
	MD5_GG (d, a, b, c, x[ 2], _S22, 0xfcefa3f8); /* 30 */
	MD5_GG (c, d, a, b, x[ 7], _S23, 0x676f02d9); /* 31 */
	MD5_GG (b, c, d, a, x[12], _S24, 0x8d2a4c8a); /* 32 */

	// Round 3
	MD5_HH (a, b, c, d, x[ 5], _S31, 0xfffa3942); /* 33 */
	MD5_HH (d, a, b, c, x[ 8], _S32, 0x8771f681); /* 34 */
	MD5_HH (c, d, a, b, x[11], _S33, 0x6d9d6122); /* 35 */
	MD5_HH (b, c, d, a, x[14], _S34, 0xfde5380c); /* 36 */
	MD5_HH (a, b, c, d, x[ 1], _S31, 0xa4beea44); /* 37 */
	MD5_HH (d, a, b, c, x[ 4], _S32, 0x4bdecfa9); /* 38 */
	MD5_HH (c, d, a, b, x[ 7], _S33, 0xf6bb4b60); /* 39 */
	MD5_HH (b, c, d, a, x[10], _S34, 0xbebfbc70); /* 40 */
	MD5_HH (a, b, c, d, x[13], _S31, 0x289b7ec6); /* 41 */
	MD5_HH (d, a, b, c, x[ 0], _S32, 0xeaa127fa); /* 42 */
	MD5_HH (c, d, a, b, x[ 3], _S33, 0xd4ef3085); /* 43 */
	MD5_HH (b, c, d, a, x[ 6], _S34,  0x4881d05); /* 44 */
	MD5_HH (a, b, c, d, x[ 9], _S31, 0xd9d4d039); /* 45 */
	MD5_HH (d, a, b, c, x[12], _S32, 0xe6db99e5); /* 46 */
	MD5_HH (c, d, a, b, x[15], _S33, 0x1fa27cf8); /* 47 */
	MD5_HH (b, c, d, a, x[ 2], _S34, 0xc4ac5665); /* 48 */

	// Round 4
	MD5_II (a, b, c, d, x[ 0], _S41, 0xf4292244); /* 49 */
	MD5_II (d, a, b, c, x[ 7], _S42, 0x432aff97); /* 50 */
	MD5_II (c, d, a, b, x[14], _S43, 0xab9423a7); /* 51 */
	MD5_II (b, c, d, a, x[ 5], _S44, 0xfc93a039); /* 52 */
	MD5_II (a, b, c, d, x[12], _S41, 0x655b59c3); /* 53 */
	MD5_II (d, a, b, c, x[ 3], _S42, 0x8f0ccc92); /* 54 */
	MD5_II (c, d, a, b, x[10], _S43, 0xffeff47d); /* 55 */
	MD5_II (b, c, d, a, x[ 1], _S44, 0x85845dd1); /* 56 */
	MD5_II (a, b, c, d, x[ 8], _S41, 0x6fa87e4f); /* 57 */
	MD5_II (d, a, b, c, x[15], _S42, 0xfe2ce6e0); /* 58 */
	MD5_II (c, d, a, b, x[ 6], _S43, 0xa3014314); /* 59 */
	MD5_II (b, c, d, a, x[13], _S44, 0x4e0811a1); /* 60 */
	MD5_II (a, b, c, d, x[ 4], _S41, 0xf7537e82); /* 61 */
	MD5_II (d, a, b, c, x[11], _S42, 0xbd3af235); /* 62 */
	MD5_II (c, d, a, b, x[ 2], _S43, 0x2ad7d2bb); /* 63 */
	MD5_II (b, c, d, a, x[ 9], _S44, 0xeb86d391); /* 64 */

	state[0] += a;
	state[1] += b;
	state[2] += c;
	state[3] += d;

	// Zeroize sensitive information.
	FMemory::Memset( x, 0, sizeof(x) ); //-V597
}

void FMD5::Encode( uint8* output, const uint32* input, int32 len )
{
	int32 i, j;

	for (i = 0, j = 0; j < len; i++, j += 4) 
{
		output[j] = (uint8)(input[i] & 0xff);
		output[j+1] = (uint8)((input[i] >> 8) & 0xff);
		output[j+2] = (uint8)((input[i] >> 16) & 0xff);
		output[j+3] = (uint8)((input[i] >> 24) & 0xff);
	}
}

void FMD5::Decode( uint32* output, const uint8* input, int32 len )
{
	int32 i, j;

	for (i = 0, j = 0; j < len; i++, j += 4)
	{
		output[i] = ((uint32)input[j]) | (((uint32)input[j+1]) << 8) |
		(((uint32)input[j+2]) << 16) | (((uint32)input[j+3]) << 24);
	}
}

/// @cond DOXYGEN_WARNINGS
//
// MD5 initialization.  Begins an MD5 operation, writing a new context.
//
// void appMD5Init( FMD5Context* context )
// {
// 	context->count[0] = context->count[1] = 0;
// 	// Load magic initialization constants.
// 	context->state[0] = 0x67452301;
// 	context->state[1] = 0xefcdab89;
// 	context->state[2] = 0x98badcfe;
// 	context->state[3] = 0x10325476;
// }

//
// MD5 block update operation.  Continues an MD5 message-digest operation,
// processing another message block, and updating the context.
//
// void appMD5Update( FMD5Context* context, uint8* input, int32 inputLen )
// {
// 	int32 i, index, partLen;
// 
// 	// Compute number of bytes mod 64.
// 	index = (int32)((context->count[0] >> 3) & 0x3F);
// 
// 	// Update number of bits.
// 	if ((context->count[0] += ((uint32)inputLen << 3)) < ((uint32)inputLen << 3))
// 		context->count[1]++;
// 	context->count[1] += ((uint32)inputLen >> 29);
// 
// 	partLen = 64 - index;
// 
// 	// Transform as many times as possible.
// 	if (inputLen >= partLen) {
// 		FMemory::Memcpy( &context->buffer[index], input, partLen );
// 		appMD5Transform( context->state, context->buffer );
// 		for (i = partLen; i + 63 < inputLen; i += 64)
// 			appMD5Transform( context->state, &input[i] );
// 		index = 0;
// 	} else
// 		i = 0;
// 
// 	// Buffer remaining input.
// 	FMemory::Memcpy( &context->buffer[index], &input[i], inputLen-i );
// }

//
// MD5 finalization. Ends an MD5 message-digest operation, writing the
// the message digest and zeroizing the context.
// Digest is 16 BYTEs.
//
// void appMD5Final ( uint8* digest, FMD5Context* context )
// {
// 	uint8 bits[8];
// 	int32 index, padLen;
// 
// 	// Save number of bits.
// 	appMD5Encode( bits, context->count, 8 );
// 
// 	// Pad out to 56 mod 64.
// 	index = (int32)((context->count[0] >> 3) & 0x3f);
// 	padLen = (index < 56) ? (56 - index) : (120 - index);
// 	appMD5Update( context, PADDING, padLen );
// 
// 	// Append length (before padding).
// 	appMD5Update( context, bits, 8 );
// 
// 	// Store state in digest
// 	appMD5Encode( digest, context->state, 16 );
// 
// 	// Zeroize sensitive information.
// 	FMemory::Memset( context, 0, sizeof(*context) );
// }

//
// MD5 basic transformation. Transforms state based on block.
//
// void appMD5Transform( uint32* state, uint8* block )
// {
// 	uint32 a = state[0], b = state[1], c = state[2], d = state[3], x[16];
// 
// 	appMD5Decode( x, block, 64 );
// 
// 	// Round 1
// 	MD5_FF (a, b, c, d, x[ 0], S11, 0xd76aa478); /* 1 */
// 	MD5_FF (d, a, b, c, x[ 1], S12, 0xe8c7b756); /* 2 */
// 	MD5_FF (c, d, a, b, x[ 2], S13, 0x242070db); /* 3 */
// 	MD5_FF (b, c, d, a, x[ 3], S14, 0xc1bdceee); /* 4 */
// 	MD5_FF (a, b, c, d, x[ 4], S11, 0xf57c0faf); /* 5 */
// 	MD5_FF (d, a, b, c, x[ 5], S12, 0x4787c62a); /* 6 */
// 	MD5_FF (c, d, a, b, x[ 6], S13, 0xa8304613); /* 7 */
// 	MD5_FF (b, c, d, a, x[ 7], S14, 0xfd469501); /* 8 */
// 	MD5_FF (a, b, c, d, x[ 8], S11, 0x698098d8); /* 9 */
// 	MD5_FF (d, a, b, c, x[ 9], S12, 0x8b44f7af); /* 10 */
// 	MD5_FF (c, d, a, b, x[10], S13, 0xffff5bb1); /* 11 */
// 	MD5_FF (b, c, d, a, x[11], S14, 0x895cd7be); /* 12 */
// 	MD5_FF (a, b, c, d, x[12], S11, 0x6b901122); /* 13 */
// 	MD5_FF (d, a, b, c, x[13], S12, 0xfd987193); /* 14 */
// 	MD5_FF (c, d, a, b, x[14], S13, 0xa679438e); /* 15 */
// 	MD5_FF (b, c, d, a, x[15], S14, 0x49b40821); /* 16 */
// 
// 	// Round 2
// 	MD5_GG (a, b, c, d, x[ 1], S21, 0xf61e2562); /* 17 */
// 	MD5_GG (d, a, b, c, x[ 6], S22, 0xc040b340); /* 18 */
// 	MD5_GG (c, d, a, b, x[11], S23, 0x265e5a51); /* 19 */
// 	MD5_GG (b, c, d, a, x[ 0], S24, 0xe9b6c7aa); /* 20 */
// 	MD5_GG (a, b, c, d, x[ 5], S21, 0xd62f105d); /* 21 */
// 	MD5_GG (d, a, b, c, x[10], S22,  0x2441453); /* 22 */
// 	MD5_GG (c, d, a, b, x[15], S23, 0xd8a1e681); /* 23 */
// 	MD5_GG (b, c, d, a, x[ 4], S24, 0xe7d3fbc8); /* 24 */
// 	MD5_GG (a, b, c, d, x[ 9], S21, 0x21e1cde6); /* 25 */
// 	MD5_GG (d, a, b, c, x[14], S22, 0xc33707d6); /* 26 */
// 	MD5_GG (c, d, a, b, x[ 3], S23, 0xf4d50d87); /* 27 */
// 	MD5_GG (b, c, d, a, x[ 8], S24, 0x455a14ed); /* 28 */
// 	MD5_GG (a, b, c, d, x[13], S21, 0xa9e3e905); /* 29 */
// 	MD5_GG (d, a, b, c, x[ 2], S22, 0xfcefa3f8); /* 30 */
// 	MD5_GG (c, d, a, b, x[ 7], S23, 0x676f02d9); /* 31 */
// 	MD5_GG (b, c, d, a, x[12], S24, 0x8d2a4c8a); /* 32 */
// 
// 	// Round 3
// 	MD5_HH (a, b, c, d, x[ 5], S31, 0xfffa3942); /* 33 */
// 	MD5_HH (d, a, b, c, x[ 8], S32, 0x8771f681); /* 34 */
// 	MD5_HH (c, d, a, b, x[11], S33, 0x6d9d6122); /* 35 */
// 	MD5_HH (b, c, d, a, x[14], S34, 0xfde5380c); /* 36 */
// 	MD5_HH (a, b, c, d, x[ 1], S31, 0xa4beea44); /* 37 */
// 	MD5_HH (d, a, b, c, x[ 4], S32, 0x4bdecfa9); /* 38 */
// 	MD5_HH (c, d, a, b, x[ 7], S33, 0xf6bb4b60); /* 39 */
// 	MD5_HH (b, c, d, a, x[10], S34, 0xbebfbc70); /* 40 */
// 	MD5_HH (a, b, c, d, x[13], S31, 0x289b7ec6); /* 41 */
// 	MD5_HH (d, a, b, c, x[ 0], S32, 0xeaa127fa); /* 42 */
// 	MD5_HH (c, d, a, b, x[ 3], S33, 0xd4ef3085); /* 43 */
// 	MD5_HH (b, c, d, a, x[ 6], S34,  0x4881d05); /* 44 */
// 	MD5_HH (a, b, c, d, x[ 9], S31, 0xd9d4d039); /* 45 */
// 	MD5_HH (d, a, b, c, x[12], S32, 0xe6db99e5); /* 46 */
// 	MD5_HH (c, d, a, b, x[15], S33, 0x1fa27cf8); /* 47 */
// 	MD5_HH (b, c, d, a, x[ 2], S34, 0xc4ac5665); /* 48 */
// 
// 	// Round 4
// 	MD5_II (a, b, c, d, x[ 0], S41, 0xf4292244); /* 49 */
// 	MD5_II (d, a, b, c, x[ 7], S42, 0x432aff97); /* 50 */
// 	MD5_II (c, d, a, b, x[14], S43, 0xab9423a7); /* 51 */
// 	MD5_II (b, c, d, a, x[ 5], S44, 0xfc93a039); /* 52 */
// 	MD5_II (a, b, c, d, x[12], S41, 0x655b59c3); /* 53 */
// 	MD5_II (d, a, b, c, x[ 3], S42, 0x8f0ccc92); /* 54 */
// 	MD5_II (c, d, a, b, x[10], S43, 0xffeff47d); /* 55 */
// 	MD5_II (b, c, d, a, x[ 1], S44, 0x85845dd1); /* 56 */
// 	MD5_II (a, b, c, d, x[ 8], S41, 0x6fa87e4f); /* 57 */
// 	MD5_II (d, a, b, c, x[15], S42, 0xfe2ce6e0); /* 58 */
// 	MD5_II (c, d, a, b, x[ 6], S43, 0xa3014314); /* 59 */
// 	MD5_II (b, c, d, a, x[13], S44, 0x4e0811a1); /* 60 */
// 	MD5_II (a, b, c, d, x[ 4], S41, 0xf7537e82); /* 61 */
// 	MD5_II (d, a, b, c, x[11], S42, 0xbd3af235); /* 62 */
// 	MD5_II (c, d, a, b, x[ 2], S43, 0x2ad7d2bb); /* 63 */
// 	MD5_II (b, c, d, a, x[ 9], S44, 0xeb86d391); /* 64 */
// 
// 	state[0] += a;
// 	state[1] += b;
// 	state[2] += c;
// 	state[3] += d;
// 
// 	// Zeroize sensitive information.
// 	FMemory::Memset( x, 0, sizeof(x) );
// }

//
// Encodes input (uint32) into output (uint8).
// Assumes len is a multiple of 4.
//
// void appMD5Encode( uint8* output, uint32* input, int32 len )
// {
// 	int32 i, j;
// 
// 	for (i = 0, j = 0; j < len; i++, j += 4) {
// 		output[j] = (uint8)(input[i] & 0xff);
// 		output[j+1] = (uint8)((input[i] >> 8) & 0xff);
// 		output[j+2] = (uint8)((input[i] >> 16) & 0xff);
// 		output[j+3] = (uint8)((input[i] >> 24) & 0xff);
// 	}
// }

//
// Decodes input (uint8) into output (uint32).
// Assumes len is a multiple of 4.
//
// void appMD5Decode( uint32* output, uint8* input, int32 len )
// {
// 	int32 i, j;
// 
// 	for (i = 0, j = 0; j < len; i++, j += 4)
// 		output[i] = ((uint32)input[j]) | (((uint32)input[j+1]) << 8) |
// 		(((uint32)input[j+2]) << 16) | (((uint32)input[j+3]) << 24);
// }
/// @endcond

FString FMD5Hash::LexToString() const
{
	if (!bIsValid)
	{
		return FString();
	}

	return FString::Printf(TEXT("%02x%02x%02x%02x%02x%02x%02x%02x%02x%02x%02x%02x%02x%02x%02x%02x"),
		Bytes[0], Bytes[1], Bytes[2], Bytes[3], Bytes[4], Bytes[5], Bytes[6], Bytes[7],
		Bytes[8], Bytes[9], Bytes[10], Bytes[11], Bytes[12], Bytes[13], Bytes[14], Bytes[15]);
}

void FMD5Hash::LexFromString(const TCHAR* Buffer)
{
	auto HexCharacterToDecimalValue = [](const TCHAR InHexChar, uint8& OutDecValue) -> bool
	{
		TCHAR Base = TEXT('\0');
		if (InHexChar >= '0' && InHexChar <= '9')
		{
			OutDecValue = (uint8)(InHexChar - '0');
		}
		else if (InHexChar >= 'A' && InHexChar <= 'F')
		{
			OutDecValue = (uint8)((InHexChar - 'A') + 10);
		}
		else if (InHexChar >= 'a' && InHexChar <= 'f')
		{
			OutDecValue = (uint8)((InHexChar - 'a') + 10);
		}
		else
		{
			return false;
		}

		return true;
	};

	uint8 NewBytes[16];
	for (int32 ByteIndex = 0, BufferIndex = 0; ByteIndex < 16; ++ByteIndex)
	{
		const TCHAR FirstChar = Buffer[BufferIndex++];
		if (FirstChar == '\0')
		{
			return;
		}

		const TCHAR SecondChar = Buffer[BufferIndex++];
		if (SecondChar == '\0')
		{
			return;
		}

		uint8 FirstCharVal, SecondCharVal;
		if (!HexCharacterToDecimalValue(FirstChar, FirstCharVal) || !HexCharacterToDecimalValue(SecondChar, SecondCharVal))
		{
			return;
		}

		NewBytes[ByteIndex] = (uint8)((FirstCharVal << 4) + SecondCharVal);
	}

	FMemory::Memcpy(Bytes, NewBytes, 16);
	bIsValid = true;
}

FMD5Hash FMD5Hash::HashFile(const TCHAR* InFilename, TArray<uint8>* Buffer)
{
	FArchive* Ar = IFileManager::Get().CreateFileReader(InFilename);
	FMD5Hash Result = HashFileFromArchive( Ar, Buffer );
	delete Ar;

	return Result;
}

FMD5Hash FMD5Hash::HashFileFromArchive( FArchive* Ar, TArray<uint8>* Buffer)
{
	FMD5Hash Hash;
	if (Ar)
	{
		TArray<uint8> LocalScratch;
		if (!Buffer)
		{
			LocalScratch.SetNumUninitialized(1024 * 64);
			Buffer = &LocalScratch; //-V506
		}
		FMD5 MD5;

		const int64 Size = Ar->TotalSize();
		int64 Position = 0;

		// Read in BufferSize chunks
		while (Position < Size)
		{
			const auto ReadNum = FMath::Min(Size - Position, (int64)Buffer->Num());
			Ar->Serialize(Buffer->GetData(), ReadNum);
			MD5.Update(Buffer->GetData(), ReadNum);

			Position += ReadNum;
		}

		Hash.Set(MD5);
	}

	return Hash;
}

FCbWriter& FMD5Hash::WriteCompactBinary(FCbWriter& Writer) const
{
	// TODO: Should we write bIsValid as well?
	Writer.AddBinary(MakeMemoryView(Bytes, GetSize()));
	return Writer;
}

bool FMD5Hash::LoadFromCompactBinary(const FCbFieldView& Field)
{
	FMemoryView BinaryView = const_cast<FCbFieldView&>(Field).AsBinaryView();
	if (BinaryView.GetSize() != GetSize())
	{
		bIsValid = false;
		return false;
	}

	FMemory::Memcpy(Bytes, BinaryView.GetData(), GetSize());
	bIsValid = true;
	return true;
}



/*-----------------------------------------------------------------------------
	SHA-1
-----------------------------------------------------------------------------*/

/** Global maps of filename to hash value */
TMap<FString, uint8*> FSHA1::FullFileSHAHashMap;
TMap<FString, uint8*> FSHA1::ScriptSHAHashMap;

// Rotate x bits to the left
#ifndef ROL32
#ifdef _MSC_VER
#define ROL32(_val32, _nBits) _rotl(_val32, _nBits)
#else
#define ROL32(_val32, _nBits) (((_val32)<<(_nBits))|((_val32)>>(32-(_nBits))))
#endif
#endif

#if PLATFORM_LITTLE_ENDIAN
#define SHABLK0(i) (block[i] = (ROL32(block[i],24) & 0xFF00FF00) | (ROL32(block[i],8) & 0x00FF00FF))
#else
#define SHABLK0(i) (block[i])
#endif

#define SHABLK(i) (block[i&15] = ROL32(block[(i+13)&15] ^ block[(i+8)&15] ^ block[(i+2)&15] ^ block[i&15],1))

// SHA-1 rounds
#define _R0(v,w,x,y,z,i) { z+=((w&(x^y))^y)+SHABLK0(i)+0x5A827999+ROL32(v,5); w=ROL32(w,30); }
#define _R1(v,w,x,y,z,i) { z+=((w&(x^y))^y)+SHABLK(i)+0x5A827999+ROL32(v,5); w=ROL32(w,30); }
#define _R2(v,w,x,y,z,i) { z+=(w^x^y)+SHABLK(i)+0x6ED9EBA1+ROL32(v,5); w=ROL32(w,30); }
#define _R3(v,w,x,y,z,i) { z+=(((w|x)&y)|(w&x))+SHABLK(i)+0x8F1BBCDC+ROL32(v,5); w=ROL32(w,30); }
#define _R4(v,w,x,y,z,i) { z+=(w^x^y)+SHABLK(i)+0xCA62C1D6+ROL32(v,5); w=ROL32(w,30); }

#define SHA1_BLOCK_SIZE 64

bool FSHAHash::LoadFromCompactBinary(const FCbFieldView& Field)
{
	FMemoryView Bytes = const_cast<FCbFieldView&>(Field).AsBinaryView();
	if (Bytes.GetSize() != sizeof(Hash))
	{
		*this = FSHAHash();
		return false;
	}
	FMemory::Memcpy(Hash, Bytes.GetData(), Bytes.GetSize());
	return true;
}

FCbWriter& FSHAHash::WriteCompactBinary(FCbWriter& Writer) const
{
	Writer.AddBinary(FMemoryView(&Hash, sizeof(Hash)));
	return Writer;
}

void Freeze::IntrinsicToString(const FSHAHash& Object, const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams, FMemoryToStringContext& OutContext)
{
	OutContext.String->Appendf(TEXT("%s\n"), *Object.ToString());
}

FSHA1::FSHA1()
{
	Reset();
}

FSHA1::~FSHA1()
{
	Reset();
}

void FSHA1::Reset()
{
	// SHA1 initialization constants
	m_state[0] = 0x67452301;
	m_state[1] = 0xEFCDAB89;
	m_state[2] = 0x98BADCFE;
	m_state[3] = 0x10325476;
	m_state[4] = 0xC3D2E1F0;

	// input byte count
	m_count = 0;
}

#if UE_PLATFORM_SHA_X86

static inline bool DetectShaInstructions()
{
	int info1[4];
	int info7[4];
#if PLATFORM_COMPILER_CLANG
	__cpuid(1, info1[0], info1[1], info1[2], info1[3]);
	__cpuid_count(7, 0, info7[0], info7[1], info7[2], info7[3]);
#else
	__cpuid(info1, 1);
	__cpuidex(info7, 7, 0);
#endif
	bool bHasSSSE3 = (info1[2] & 0x00000200) != 0;
	bool bHasSHA = (info7[1] & 0x20000000) != 0;
	return bHasSSSE3 && bHasSHA;
}

static bool CanUseShaInstructions()
{
#if UE_PLATFORM_SHA_FALLBACK
	// run cpuid only once
	static bool bCanUse = DetectShaInstructions();
	return bCanUse;
#else
	// no SHA fallback code means SHA instructions will be used unconditionally
	return true;
#endif
}

#if PLATFORM_COMPILER_CLANG
#pragma clang attribute push (__attribute__((target("sha"))), apply_to=function)
#endif

static void Sha1TransformX86(uint32* State, const void* Buffer, uint64 Size)
{
	// https://www.intel.com/content/www/us/en/developer/articles/technical/intel-sha-extensions.html
	// https://www.intel.com/content/www/us/en/docs/intrinsics-guide/index.html#othertechs=SHA

	// to reverse byte order
	__m128i REVERSE_BYTES = _mm_setr_epi8(
		0x0f, 0x0e, 0x0d, 0x0c, 0x0b, 0x0a, 0x09, 0x08,
		0x07, 0x06, 0x05, 0x04, 0x03, 0x02, 0x01, 0x00);

	// load current state
	__m128i ABCD = _mm_loadu_si128((const __m128i*)State);
	__m128i E0 = _mm_setr_epi32(0, 0, 0, State[4]);
	ABCD = _mm_shuffle_epi32(ABCD, _MM_SHUFFLE(0, 1, 2, 3));

	const __m128i* Data = (const __m128i*)Buffer;
	while (Size >= SHA1_BLOCK_SIZE)
	{
		__m128i E1;

		// remember current state
		__m128i ABCD_SAVE = ABCD;
		__m128i E_SAVE = E0;

		// load message from buffer as 32-bit big-endian integers
		__m128i M0 = _mm_loadu_si128(Data + 0);
		__m128i M1 = _mm_loadu_si128(Data + 1);
		__m128i M2 = _mm_loadu_si128(Data + 2);
		__m128i M3 = _mm_loadu_si128(Data + 3);
		M0 = _mm_shuffle_epi8(M0, REVERSE_BYTES);
		M1 = _mm_shuffle_epi8(M1, REVERSE_BYTES);
		M2 = _mm_shuffle_epi8(M2, REVERSE_BYTES);
		M3 = _mm_shuffle_epi8(M3, REVERSE_BYTES);

		// 0..3
		E0 = _mm_add_epi32(E0, M0);
		E1 = ABCD;
		ABCD = _mm_sha1rnds4_epu32(ABCD, E0, 0);

		// 4..7
		E1 = _mm_sha1nexte_epu32(E1, M1);
		E0 = ABCD;
		ABCD = _mm_sha1rnds4_epu32(ABCD, E1, 0);
		M0 = _mm_sha1msg1_epu32(M0, M1);

		// 8..11
		E0 = _mm_sha1nexte_epu32(E0, M2);
		E1 = ABCD;
		ABCD = _mm_sha1rnds4_epu32(ABCD, E0, 0);
		M1 = _mm_sha1msg1_epu32(M1, M2);
		M0 = _mm_xor_si128(M0, M2);

		// 12..15
		E1 = _mm_sha1nexte_epu32(E1, M3);
		E0 = ABCD;
		M0 = _mm_sha1msg2_epu32(M0, M3);
		ABCD = _mm_sha1rnds4_epu32(ABCD, E1, 0);
		M2 = _mm_sha1msg1_epu32(M2, M3);
		M1 = _mm_xor_si128(M1, M3);

		// 16..19
		E0 = _mm_sha1nexte_epu32(E0, M0);
		E1 = ABCD;
		M1 = _mm_sha1msg2_epu32(M1, M0);
		ABCD = _mm_sha1rnds4_epu32(ABCD, E0, 0);
		M3 = _mm_sha1msg1_epu32(M3, M0);
		M2 = _mm_xor_si128(M2, M0);

		// 20..23
		E1 = _mm_sha1nexte_epu32(E1, M1);
		E0 = ABCD;
		M2 = _mm_sha1msg2_epu32(M2, M1);
		ABCD = _mm_sha1rnds4_epu32(ABCD, E1, 1);
		M0 = _mm_sha1msg1_epu32(M0, M1);
		M3 = _mm_xor_si128(M3, M1);

		// 24..27
		E0 = _mm_sha1nexte_epu32(E0, M2);
		E1 = ABCD;
		M3 = _mm_sha1msg2_epu32(M3, M2);
		ABCD = _mm_sha1rnds4_epu32(ABCD, E0, 1);
		M1 = _mm_sha1msg1_epu32(M1, M2);
		M0 = _mm_xor_si128(M0, M2);

		// 28..31
		E1 = _mm_sha1nexte_epu32(E1, M3);
		E0 = ABCD;
		M0 = _mm_sha1msg2_epu32(M0, M3);
		ABCD = _mm_sha1rnds4_epu32(ABCD, E1, 1);
		M2 = _mm_sha1msg1_epu32(M2, M3);
		M1 = _mm_xor_si128(M1, M3);

		// 32..35
		E0 = _mm_sha1nexte_epu32(E0, M0);
		E1 = ABCD;
		M1 = _mm_sha1msg2_epu32(M1, M0);
		ABCD = _mm_sha1rnds4_epu32(ABCD, E0, 1);
		M3 = _mm_sha1msg1_epu32(M3, M0);
		M2 = _mm_xor_si128(M2, M0);

		// 36..39
		E1 = _mm_sha1nexte_epu32(E1, M1);
		E0 = ABCD;
		M2 = _mm_sha1msg2_epu32(M2, M1);
		ABCD = _mm_sha1rnds4_epu32(ABCD, E1, 1);
		M0 = _mm_sha1msg1_epu32(M0, M1);
		M3 = _mm_xor_si128(M3, M1);

		// 40..43
		E0 = _mm_sha1nexte_epu32(E0, M2);
		E1 = ABCD;
		M3 = _mm_sha1msg2_epu32(M3, M2);
		ABCD = _mm_sha1rnds4_epu32(ABCD, E0, 2);
		M1 = _mm_sha1msg1_epu32(M1, M2);
		M0 = _mm_xor_si128(M0, M2);

		// 44..47
		E1 = _mm_sha1nexte_epu32(E1, M3);
		E0 = ABCD;
		M0 = _mm_sha1msg2_epu32(M0, M3);
		ABCD = _mm_sha1rnds4_epu32(ABCD, E1, 2);
		M2 = _mm_sha1msg1_epu32(M2, M3);
		M1 = _mm_xor_si128(M1, M3);

		// 48..51
		E0 = _mm_sha1nexte_epu32(E0, M0);
		E1 = ABCD;
		M1 = _mm_sha1msg2_epu32(M1, M0);
		ABCD = _mm_sha1rnds4_epu32(ABCD, E0, 2);
		M3 = _mm_sha1msg1_epu32(M3, M0);
		M2 = _mm_xor_si128(M2, M0);

		// 52..55
		E1 = _mm_sha1nexte_epu32(E1, M1);
		E0 = ABCD;
		M2 = _mm_sha1msg2_epu32(M2, M1);
		ABCD = _mm_sha1rnds4_epu32(ABCD, E1, 2);
		M0 = _mm_sha1msg1_epu32(M0, M1);
		M3 = _mm_xor_si128(M3, M1);

		// 56..59
		E0 = _mm_sha1nexte_epu32(E0, M2);
		E1 = ABCD;
		M3 = _mm_sha1msg2_epu32(M3, M2);
		ABCD = _mm_sha1rnds4_epu32(ABCD, E0, 2);
		M1 = _mm_sha1msg1_epu32(M1, M2);
		M0 = _mm_xor_si128(M0, M2);

		// 60..63
		E1 = _mm_sha1nexte_epu32(E1, M3);
		E0 = ABCD;
		M0 = _mm_sha1msg2_epu32(M0, M3);
		ABCD = _mm_sha1rnds4_epu32(ABCD, E1, 3);
		M2 = _mm_sha1msg1_epu32(M2, M3);
		M1 = _mm_xor_si128(M1, M3);

		// 64..67
		E0 = _mm_sha1nexte_epu32(E0, M0);
		E1 = ABCD;
		M1 = _mm_sha1msg2_epu32(M1, M0);
		ABCD = _mm_sha1rnds4_epu32(ABCD, E0, 3);
		M3 = _mm_sha1msg1_epu32(M3, M0);
		M2 = _mm_xor_si128(M2, M0);

		// 68..71
		E1 = _mm_sha1nexte_epu32(E1, M1);
		E0 = ABCD;
		M2 = _mm_sha1msg2_epu32(M2, M1);
		ABCD = _mm_sha1rnds4_epu32(ABCD, E1, 3);
		M3 = _mm_xor_si128(M3, M1);

		// 72..75
		E0 = _mm_sha1nexte_epu32(E0, M2);
		E1 = ABCD;
		M3 = _mm_sha1msg2_epu32(M3, M2);
		ABCD = _mm_sha1rnds4_epu32(ABCD, E0, 3);

		// 76..79
		E1 = _mm_sha1nexte_epu32(E1, M3);
		E0 = ABCD;
		ABCD = _mm_sha1rnds4_epu32(ABCD, E1, 3);

		// add the working vars back into state
		E0 = _mm_sha1nexte_epu32(E0, E_SAVE);
		ABCD = _mm_add_epi32(ABCD, ABCD_SAVE);

		Data += 4;
		Size -= SHA1_BLOCK_SIZE;
	}

	// save state
	ABCD = _mm_shuffle_epi32(ABCD, _MM_SHUFFLE(0, 1, 2, 3));
	_mm_storeu_si128((__m128i*)State, ABCD);
	State[4] = _mm_extract_epi32(E0, 3);
}

#if PLATFORM_COMPILER_CLANG
#pragma clang attribute pop
#endif

#endif

#if UE_PLATFORM_SHA_ARMV8

static void Sha1TransformARMV8(uint32* State, const uint8_t* Buffer, uint64 Size)
{
	// https://developer.arm.com/architectures/instruction-sets/intrinsics/#f:@navigationhierarchiesinstructiongroup=[Cryptography,SHA1]

	// load current state
	uint32x4_t ABCD = vld1q_u32(State);
	uint32 E0 = State[4];

	while (Size >= SHA1_BLOCK_SIZE)
	{
		// remember current state
		uint32x4_t ABCD_SAVE = ABCD;
		uint32_t E0_SAVE = E0;

		// load message from buffer as 32-bit big-endian integers
		uint32x4_t M0 = vreinterpretq_u32_u8(vrev32q_u8(vld1q_u8(Buffer + 0 * 16)));
		uint32x4_t M1 = vreinterpretq_u32_u8(vrev32q_u8(vld1q_u8(Buffer + 1 * 16)));
		uint32x4_t M2 = vreinterpretq_u32_u8(vrev32q_u8(vld1q_u8(Buffer + 2 * 16)));
		uint32x4_t M3 = vreinterpretq_u32_u8(vrev32q_u8(vld1q_u8(Buffer + 3 * 16)));

		uint32x4_t T0 = vaddq_u32(M0, vdupq_n_u32(0x5A827999));
		uint32x4_t T1 = vaddq_u32(M1, vdupq_n_u32(0x5A827999));
		uint32 E1;

		// 0..3
		E1 = vsha1h_u32(vgetq_lane_u32(ABCD, 0));
		ABCD = vsha1cq_u32(ABCD, E0, T0);
		T0 = vaddq_u32(M2, vdupq_n_u32(0x5A827999));
		M0 = vsha1su0q_u32(M0, M1, M2);

		// 4..7
		E0 = vsha1h_u32(vgetq_lane_u32(ABCD, 0));
		ABCD = vsha1cq_u32(ABCD, E1, T1);
		T1 = vaddq_u32(M3, vdupq_n_u32(0x5A827999));
		M0 = vsha1su1q_u32(M0, M3);
		M1 = vsha1su0q_u32(M1, M2, M3);

		// 8..11
		E1 = vsha1h_u32(vgetq_lane_u32(ABCD, 0));
		ABCD = vsha1cq_u32(ABCD, E0, T0);
		T0 = vaddq_u32(M0, vdupq_n_u32(0x5A827999));
		M1 = vsha1su1q_u32(M1, M0);
		M2 = vsha1su0q_u32(M2, M3, M0);

		// 12..15
		E0 = vsha1h_u32(vgetq_lane_u32(ABCD, 0));
		ABCD = vsha1cq_u32(ABCD, E1, T1);
		T1 = vaddq_u32(M1, vdupq_n_u32(0x6ED9EBA1));
		M2 = vsha1su1q_u32(M2, M1);
		M3 = vsha1su0q_u32(M3, M0, M1);

		// 16..19
		E1 = vsha1h_u32(vgetq_lane_u32(ABCD, 0));
		ABCD = vsha1cq_u32(ABCD, E0, T0);
		T0 = vaddq_u32(M2, vdupq_n_u32(0x6ED9EBA1));
		M3 = vsha1su1q_u32(M3, M2);
		M0 = vsha1su0q_u32(M0, M1, M2);

		// 20..23
		E0 = vsha1h_u32(vgetq_lane_u32(ABCD, 0));
		ABCD = vsha1pq_u32(ABCD, E1, T1);
		T1 = vaddq_u32(M3, vdupq_n_u32(0x6ED9EBA1));
		M0 = vsha1su1q_u32(M0, M3);
		M1 = vsha1su0q_u32(M1, M2, M3);

		// 24..27
		E1 = vsha1h_u32(vgetq_lane_u32(ABCD, 0));
		ABCD = vsha1pq_u32(ABCD, E0, T0);
		T0 = vaddq_u32(M0, vdupq_n_u32(0x6ED9EBA1));
		M1 = vsha1su1q_u32(M1, M0);
		M2 = vsha1su0q_u32(M2, M3, M0);

		// 28..31
		E0 = vsha1h_u32(vgetq_lane_u32(ABCD, 0));
		ABCD = vsha1pq_u32(ABCD, E1, T1);
		T1 = vaddq_u32(M1, vdupq_n_u32(0x6ED9EBA1));
		M2 = vsha1su1q_u32(M2, M1);
		M3 = vsha1su0q_u32(M3, M0, M1);

		// 32..35
		E1 = vsha1h_u32(vgetq_lane_u32(ABCD, 0));
		ABCD = vsha1pq_u32(ABCD, E0, T0);
		T0 = vaddq_u32(M2, vdupq_n_u32(0x8F1BBCDC));
		M3 = vsha1su1q_u32(M3, M2);
		M0 = vsha1su0q_u32(M0, M1, M2);

		// 36..39
		E0 = vsha1h_u32(vgetq_lane_u32(ABCD, 0));
		ABCD = vsha1pq_u32(ABCD, E1, T1);
		T1 = vaddq_u32(M3, vdupq_n_u32(0x8F1BBCDC));
		M0 = vsha1su1q_u32(M0, M3);
		M1 = vsha1su0q_u32(M1, M2, M3);

		// 40..43
		E1 = vsha1h_u32(vgetq_lane_u32(ABCD, 0));
		ABCD = vsha1mq_u32(ABCD, E0, T0);
		T0 = vaddq_u32(M0, vdupq_n_u32(0x8F1BBCDC));
		M1 = vsha1su1q_u32(M1, M0);
		M2 = vsha1su0q_u32(M2, M3, M0);

		// 44..47
		E0 = vsha1h_u32(vgetq_lane_u32(ABCD, 0));
		ABCD = vsha1mq_u32(ABCD, E1, T1);
		T1 = vaddq_u32(M1, vdupq_n_u32(0x8F1BBCDC));
		M2 = vsha1su1q_u32(M2, M1);
		M3 = vsha1su0q_u32(M3, M0, M1);

		// 48..51
		E1 = vsha1h_u32(vgetq_lane_u32(ABCD, 0));
		ABCD = vsha1mq_u32(ABCD, E0, T0);
		T0 = vaddq_u32(M2, vdupq_n_u32(0x8F1BBCDC));
		M3 = vsha1su1q_u32(M3, M2);
		M0 = vsha1su0q_u32(M0, M1, M2);

		// 52..55
		E0 = vsha1h_u32(vgetq_lane_u32(ABCD, 0));
		ABCD = vsha1mq_u32(ABCD, E1, T1);
		T1 = vaddq_u32(M3, vdupq_n_u32(0xCA62C1D6));
		M0 = vsha1su1q_u32(M0, M3);
		M1 = vsha1su0q_u32(M1, M2, M3);

		// 56..59
		E1 = vsha1h_u32(vgetq_lane_u32(ABCD, 0));
		ABCD = vsha1mq_u32(ABCD, E0, T0);
		T0 = vaddq_u32(M0, vdupq_n_u32(0xCA62C1D6));
		M1 = vsha1su1q_u32(M1, M0);
		M2 = vsha1su0q_u32(M2, M3, M0);

		// 60..63
		E0 = vsha1h_u32(vgetq_lane_u32(ABCD, 0));
		ABCD = vsha1pq_u32(ABCD, E1, T1);
		T1 = vaddq_u32(M1, vdupq_n_u32(0xCA62C1D6));
		M2 = vsha1su1q_u32(M2, M1);
		M3 = vsha1su0q_u32(M3, M0, M1);

		// 64..67
		E1 = vsha1h_u32(vgetq_lane_u32(ABCD, 0));
		ABCD = vsha1pq_u32(ABCD, E0, T0);
		T0 = vaddq_u32(M2, vdupq_n_u32(0xCA62C1D6));
		M3 = vsha1su1q_u32(M3, M2);
		M0 = vsha1su0q_u32(M0, M1, M2);

		// 68..71
		E0 = vsha1h_u32(vgetq_lane_u32(ABCD, 0));
		ABCD = vsha1pq_u32(ABCD, E1, T1);
		T1 = vaddq_u32(M3, vdupq_n_u32(0xCA62C1D6));
		M0 = vsha1su1q_u32(M0, M3);

		// 72..75
		E1 = vsha1h_u32(vgetq_lane_u32(ABCD, 0));
		ABCD = vsha1pq_u32(ABCD, E0, T0);

		// 76..79
		E0 = vsha1h_u32(vgetq_lane_u32(ABCD, 0));
		ABCD = vsha1pq_u32(ABCD, E1, T1);

		// add the working vars back into state
		E0 += E0_SAVE;
		ABCD = vaddq_u32(ABCD_SAVE, ABCD);

		Buffer += SHA1_BLOCK_SIZE;
		Size -= SHA1_BLOCK_SIZE;
	}

	// save state
	vst1q_u32(State, ABCD);
	State[4] = E0;
}

#endif

void FSHA1::Transform(const uint8* buffer, uint64 len)
{
#if UE_PLATFORM_SHA_X86
	if (CanUseShaInstructions())
	{
		Sha1TransformX86(m_state, buffer, len);
		return;
	}
#endif

#if UE_PLATFORM_SHA_ARMV8
	Sha1TransformARMV8(m_state, buffer, len);
	return;
#endif

#if UE_PLATFORM_SHA_FALLBACK

	// load current state
	uint32 a = m_state[0];
	uint32 b = m_state[1];
	uint32 c = m_state[2];
	uint32 d = m_state[3];
	uint32 e = m_state[4];

	while (len >= SHA1_BLOCK_SIZE)
	{
		uint32 block[16];
		FMemory::Memcpy(block, buffer, SHA1_BLOCK_SIZE);

		// remember current state
		uint32 a0 = a;
		uint32 b0 = b;
		uint32 c0 = c;
		uint32 d0 = d;
		uint32 e0 = e;

		// 4 rounds of 20 operations each. Loop unrolled.
		_R0(a,b,c,d,e, 0); _R0(e,a,b,c,d, 1); _R0(d,e,a,b,c, 2); _R0(c,d,e,a,b, 3);
		_R0(b,c,d,e,a, 4); _R0(a,b,c,d,e, 5); _R0(e,a,b,c,d, 6); _R0(d,e,a,b,c, 7);
		_R0(c,d,e,a,b, 8); _R0(b,c,d,e,a, 9); _R0(a,b,c,d,e,10); _R0(e,a,b,c,d,11);
		_R0(d,e,a,b,c,12); _R0(c,d,e,a,b,13); _R0(b,c,d,e,a,14); _R0(a,b,c,d,e,15);
		_R1(e,a,b,c,d,16); _R1(d,e,a,b,c,17); _R1(c,d,e,a,b,18); _R1(b,c,d,e,a,19);
		_R2(a,b,c,d,e,20); _R2(e,a,b,c,d,21); _R2(d,e,a,b,c,22); _R2(c,d,e,a,b,23);
		_R2(b,c,d,e,a,24); _R2(a,b,c,d,e,25); _R2(e,a,b,c,d,26); _R2(d,e,a,b,c,27);
		_R2(c,d,e,a,b,28); _R2(b,c,d,e,a,29); _R2(a,b,c,d,e,30); _R2(e,a,b,c,d,31);
		_R2(d,e,a,b,c,32); _R2(c,d,e,a,b,33); _R2(b,c,d,e,a,34); _R2(a,b,c,d,e,35);
		_R2(e,a,b,c,d,36); _R2(d,e,a,b,c,37); _R2(c,d,e,a,b,38); _R2(b,c,d,e,a,39);
		_R3(a,b,c,d,e,40); _R3(e,a,b,c,d,41); _R3(d,e,a,b,c,42); _R3(c,d,e,a,b,43);
		_R3(b,c,d,e,a,44); _R3(a,b,c,d,e,45); _R3(e,a,b,c,d,46); _R3(d,e,a,b,c,47);
		_R3(c,d,e,a,b,48); _R3(b,c,d,e,a,49); _R3(a,b,c,d,e,50); _R3(e,a,b,c,d,51);
		_R3(d,e,a,b,c,52); _R3(c,d,e,a,b,53); _R3(b,c,d,e,a,54); _R3(a,b,c,d,e,55);
		_R3(e,a,b,c,d,56); _R3(d,e,a,b,c,57); _R3(c,d,e,a,b,58); _R3(b,c,d,e,a,59);
		_R4(a,b,c,d,e,60); _R4(e,a,b,c,d,61); _R4(d,e,a,b,c,62); _R4(c,d,e,a,b,63);
		_R4(b,c,d,e,a,64); _R4(a,b,c,d,e,65); _R4(e,a,b,c,d,66); _R4(d,e,a,b,c,67);
		_R4(c,d,e,a,b,68); _R4(b,c,d,e,a,69); _R4(a,b,c,d,e,70); _R4(e,a,b,c,d,71);
		_R4(d,e,a,b,c,72); _R4(c,d,e,a,b,73); _R4(b,c,d,e,a,74); _R4(a,b,c,d,e,75);
		_R4(e,a,b,c,d,76); _R4(d,e,a,b,c,77); _R4(c,d,e,a,b,78); _R4(b,c,d,e,a,79);

		// Add the working vars back into state
		a += a0;
		b += b0;
		c += c0;
		d += d0;
		e += e0;

		buffer += SHA1_BLOCK_SIZE;
		len -= SHA1_BLOCK_SIZE;
	}

	// save state
	m_state[0] = a;
	m_state[1] = b;
	m_state[2] = c;
	m_state[3] = d;
	m_state[4] = e;
#endif
}

// do not remove #undefs, or you can get name collision with libc++'s <functional>
#undef _R0
#undef _R1
#undef _R2
#undef _R3
#undef _R4
#undef SHABLK
#undef SHABLK0
#undef ROL32

// Use this function to hash in binary data
void FSHA1::Update(const uint8* data, uint64 len)
{
	if (len == 0)
	{
		return;
	}

	size_t left = m_count % SHA1_BLOCK_SIZE;
	size_t fill = SHA1_BLOCK_SIZE - left;

	m_count += len;

	if (left && len >= fill)
	{
		// process internal buffer if it can be filled fully
		FMemory::Memcpy(&m_buffer[left], data, fill);
		Transform(m_buffer, SHA1_BLOCK_SIZE);
		data += fill;
		len -= fill;
		left = 0;
	}

	if (len >= SHA1_BLOCK_SIZE)
	{
		// process whole blocks in one call as much as possible
		Transform(data, len);
		data += len & ~(SHA1_BLOCK_SIZE - 1);
		len %= SHA1_BLOCK_SIZE;
	}

	// remember any leftover bytes into internal buffer
	FMemory::Memcpy(&m_buffer[left], data, len);
}

// Use this function to hash in strings
void FSHA1::UpdateWithString(const TCHAR *String, uint32 Length)
{
	Update((const uint8*)StringCast<UCS2CHAR>(String, Length).Get(), Length * sizeof(UCS2CHAR));
}

void FSHA1::Final()
{
	// processed input bit count, in big endian
	uint8 finalcount[8];
	for (int i = 0; i < 8; i++)
	{
		finalcount[i] = (uint8)((m_count << 3) >> ((7 - i) * 8));
	}

	// pad to 56 mod 64
	size_t last = m_count % SHA1_BLOCK_SIZE;
	size_t padn = (last < SHA1_BLOCK_SIZE - 8) ? (SHA1_BLOCK_SIZE - 8 - last) : (2 * SHA1_BLOCK_SIZE - 8 - last);
	Update(PADDING, padn);

	// last 8 bytes is size
	Update(finalcount, sizeof(finalcount));

	for (int i = 0; i < 20; i++)
	{
		m_digest[i] = (uint8)(m_state[i >> 2] >> ((3 - (i & 3)) * 8));
	}
}

// Get the raw message digest
void FSHA1::GetHash(uint8 *puDest) const
{
	FMemory::Memcpy(puDest, m_digest, 20);
}

/**
* Calculate the hash on a single block and return it
*
* @param Data Input data to hash
* @param DataSize Size of the Data block
* @param OutHash Resulting hash value (20 byte buffer)
*/
void FSHA1::HashBuffer(const void* Data, uint64 DataSize, uint8* OutHash)
{
	// do an atomic hash operation
	FSHA1 Sha;
	Sha.Update((const uint8*)Data, DataSize);
	Sha.Final();
	Sha.GetHash(OutHash);
}

void FSHA1::HMACBuffer(const void* Key, uint32 KeySize, const void* Data, uint64 DataSize, uint8* OutHash)
{
	const uint8 BlockSize = 64;
	const uint8 HashSize = 20;
	uint8 FinalKey[BlockSize];

	// Fit 'Key' into a BlockSize-aligned 'FinalKey' value
	if (KeySize > BlockSize)
	{
		HashBuffer(Key, KeySize, FinalKey);

		FMemory::Memzero(FinalKey + HashSize, BlockSize - HashSize);
	}
	else if (KeySize < BlockSize)
	{
		FMemory::Memcpy(FinalKey, Key, KeySize);
		FMemory::Memzero(FinalKey + KeySize, BlockSize - KeySize);
	}
	else
	{
		FMemory::Memcpy(FinalKey, Key, KeySize);
	}


	uint8 OKeyPad[BlockSize];
	uint8 IKeyPad[BlockSize];

	for (int32 i=0; i<BlockSize; i++)
	{
		OKeyPad[i] = 0x5C ^ FinalKey[i];
		IKeyPad[i] = 0x36 ^ FinalKey[i];
	}


	// Start concatenating/hashing the pads/data etc: Hash(OKeyPad + Hash(IKeyPad + Data))
	uint8* IKeyPad_Data = new uint8[UE_ARRAY_COUNT(IKeyPad) + DataSize];

	FMemory::Memcpy(IKeyPad_Data, IKeyPad, UE_ARRAY_COUNT(IKeyPad));
	FMemory::Memcpy(IKeyPad_Data + UE_ARRAY_COUNT(IKeyPad), Data, DataSize);


	uint8 IKeyPad_Data_Hash[HashSize];

	HashBuffer(IKeyPad_Data, UE_ARRAY_COUNT(IKeyPad) + DataSize, IKeyPad_Data_Hash);

	delete[] IKeyPad_Data;


	uint8 OKeyPad_IHash[UE_ARRAY_COUNT(OKeyPad) + HashSize];

	FMemory::Memcpy(OKeyPad_IHash, OKeyPad, UE_ARRAY_COUNT(OKeyPad));
	FMemory::Memcpy(OKeyPad_IHash + UE_ARRAY_COUNT(OKeyPad), IKeyPad_Data_Hash, HashSize);


	// Output the final hash
	HashBuffer(OKeyPad_IHash, UE_ARRAY_COUNT(OKeyPad_IHash), OutHash);
}


/**
 * Shared hashes.sha reading code (each platform gets a buffer to the data,
 * then passes it to this function for processing)
 */
void FSHA1::InitializeFileHashesFromBuffer(uint8* Buffer, uint64 BufferSize, bool bDuplicateKeyMemory)
{
	// the start of the file is full file hashes
	bool bIsDoingFullFileHashes = true;
	// if it exists, parse it
	uint64 Offset = 0;
	while (Offset < BufferSize)
	{
		// format is null terminated string followed by hash
		ANSICHAR* Filename = (ANSICHAR*)Buffer + Offset;

		// make sure it's not an empty string (this could happen with an empty hash file)
		if (Filename[0])
		{
			// skip over the file
			Offset += FCStringAnsi::Strlen(Filename) + 1;

			// if we hit the magic separator between sections
			if (FCStringAnsi::Strcmp(Filename, HASHES_SHA_DIVIDER) == 0)
			{
				// switch to script sha
				bIsDoingFullFileHashes = false;

				// don't process a hash for this special case
				continue;
			}

			// duplicate the memory if needed (some hash sources are always loaded, ie in the executable,
			// so no need to duplicate that memory, we can just point into the middle of it)
			uint8* Hash;
			if (bDuplicateKeyMemory)
			{
				Hash = (uint8*)FMemory::Malloc(20);
				FMemory::Memcpy(Hash, Buffer + Offset, 20);
			}
			else
			{
				Hash = Buffer + Offset;
			}

			// offset now points to the hash data, so save a pointer to it
			(bIsDoingFullFileHashes ? FullFileSHAHashMap : ScriptSHAHashMap).Add(ANSI_TO_TCHAR(Filename), Hash);

			// move the offset over the hash (always 20 bytes)
			Offset += 20;
		}
	}

	// we should be exactly at the end
	check(Offset == BufferSize);

}


/**
 * Gets the stored SHA hash from the platform, if it exists. This function
 * must be able to be called from any thread.
 *
 * @param Pathname Pathname to the file to get the SHA for
 * @param Hash 20 byte array that receives the hash
 * @param bIsFullPackageHash true if we are looking for a full package hash, instead of a script code only hash
 *
 * @return true if the hash was found, false otherwise
 */
bool FSHA1::GetFileSHAHash(const TCHAR* Pathname, uint8 Hash[20], bool bIsFullPackageHash)
{
	// look for this file in the hash
	uint8** HashData = (bIsFullPackageHash ? FullFileSHAHashMap : ScriptSHAHashMap).Find(FPaths::GetCleanFilename(Pathname).ToLower());

	// do we want a copy?
	if (HashData && Hash)
	{
		// return it
		FMemory::Memcpy(Hash, *HashData, 20);
	}

	// return true if we found the hash
	return HashData != NULL;
}


/*-----------------------------------------------------------------------------
	FAsyncVerify.
-----------------------------------------------------------------------------*/

/**
 * Performs the async hash verification
 */
void FAsyncSHAVerify::DoWork()
{
	// default to success
	bool bFailedHashLookup = false;

	UE_LOGF(LogSHA, Log, "FAsyncSHAVerify running for hash [%ls]",*Pathname );

	// if we stored a filename to use to get the hash, get it now
	if (Pathname.Len() > 0)
	{
		// lookup the hash for the file. 
		if (FSHA1::GetFileSHAHash(*Pathname, Hash) == false)
		{
			// if it couldn't be found, then we don't calculate the hash, and we "succeed" since there's no
			// hash to check against
			bFailedHashLookup = true;
		}
	}

	bool bFailed;

	// if we have a valid hash, check it
	if (!bFailedHashLookup)
	{
		uint8 CompareHash[20];
		// hash the buffer (finally)
		FSHA1::HashBuffer(Buffer, BufferSize, CompareHash);
		
		// make sure it matches
		bFailed = FMemory::Memcmp(Hash, CompareHash, sizeof(Hash)) != 0;
	}
	else
	{
#if UE_BUILD_SHIPPING
		// if it's an error if the hash is unfound, then mark the failure. This is only done for the PC as that is an easier binary to hack
		bFailed = bIsUnfoundHashAnError;
#else
		bFailed = false; 
#endif
	}

	// delete the buffer if we should, now that we are done it
	if (bShouldDeleteBuffer)
	{
		FMemory::Free(Buffer);
	}

	// if we failed, then call the failure callback
	if (bFailed)
	{
		appOnFailSHAVerification(*Pathname, bFailedHashLookup);
	}
}

/**
 * Callback that is called if the asynchronous SHA verification fails
 * This will be called from a pooled thread.
 *
 * @param FailedPathname Pathname of file that failed to verify
 * @param bFailedDueToMissingHash true if the reason for the failure was that the hash was missing, and that was set as being an error condition
 */

/* *** NEVER CHECK THE BELOW IN SET TO 1!!! *** */
#define DISABLE_AUTHENTICATION_FOR_DEV 1    // UE - for now, we _will_ check this in with 1 because we are not generating shipping images at the moment.
/* *** NEVER CHECK THE ABOVE IN SET TO 1!!! *** */

void appOnFailSHAVerification(const TCHAR* FailedPathname, bool bFailedDueToMissingHash)
{
#if !DISABLE_AUTHENTICATION_FOR_DEV
	UE_LOGF(LogSecureHash, Fatal,"SHA Verification failed for '%ls'. Reason: %ls", 
		FailedPathname ? FailedPathname : TEXT("Unknown file"),
		bFailedDueToMissingHash ? TEXT("Missing hash") : TEXT("Bad hash"));
#else
	UE_LOGF(LogSHA, Log, "SHA Verification failed for '%ls'. Reason: %ls", 
		FailedPathname ? FailedPathname : TEXT("Unknown file"),
		bFailedDueToMissingHash ? TEXT("Missing hash") : TEXT("Bad hash"));
#endif
}

FBufferReaderWithSHA::~FBufferReaderWithSHA()
{
	Close();
}

bool FBufferReaderWithSHA::Close()
{
	// don't redo if we were already closed
	if (ReaderData)
	{
		// kick off an SHA verification task to verify. this will handle any errors we get
		(new FAutoDeleteAsyncTask<FAsyncSHAVerify>(ReaderData, ReaderSize, bFreeOnClose, *SourcePathname, bIsUnfoundHashAnError))->StartBackgroundTask();
		ReaderData = NULL;
	}

	// note that we don't allow the base class CLose to happen, as the FAsyncSHAVerify will free the buffer if needed
	return !IsError();
}
