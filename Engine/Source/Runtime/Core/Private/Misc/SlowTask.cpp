// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/SlowTask.h"

#include "Containers/Array.h"
#include "CoreTypes.h"
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformMisc.h"
#include "HAL/PlatformProcess.h"
#include "HAL/PlatformTime.h"
#include "Math/Color.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/AssertionMacros.h"
#include "Misc/FeedbackContext.h"
#include "Misc/SlowTaskStack.h"
#include "ProfilingDebugging/MiscTrace.h"

static int32 GSlowTaskMaxTraceRegionDepth = 2;
static FAutoConsoleVariableRef CVarSlowTaskMaxTraceRegionDepth (
	TEXT("Trace.SlowTaskMaxRegionDepth"),
	GSlowTaskMaxTraceRegionDepth,
	TEXT("Maximum depth of nested slow tasks to create as trace regions in Insights"),
	ECVF_Default
);

bool FSlowTask::ShouldCreateThrottledSlowTask()
{
	static double LastThrottledSlowTaskTime = 0;

	double CurrentTime = FPlatformTime::Seconds();
	if (CurrentTime - LastThrottledSlowTaskTime > 0.1)
	{
		LastThrottledSlowTaskTime = CurrentTime;
		return true;
	}

	return false;
}

FSlowTask::FSlowTask(float InAmountOfWork, const FText& InDefaultMessage, bool bInEnabled, FFeedbackContext& InContext)
	: DefaultMessage(InDefaultMessage)
	, FrameMessage()
	, TotalAmountOfWork(InAmountOfWork)
	, CompletedWork(0)
	, CurrentFrameScope(0)
	, Visibility(ESlowTaskVisibility::Default)
	, StartTime(FPlatformTime::Seconds())
	, bEnabled(bInEnabled && IsInGameThread())
	, bCreatedDialog(false)		// only set to true if we create a dialog
	, Context(InContext)
	, bSkipRecursiveDialogCreation(false)
	, RegionId(0)
{
	// If we have no work to do ourselves, create an arbitrary scope so that any actions performed underneath this still contribute to this one.
	if (TotalAmountOfWork == 0.f)
	{
		TotalAmountOfWork = CurrentFrameScope = 1.f;
	}
}

void FSlowTask::MakeRecursiveDialogIfNeeded()
{
	if (bEnabled)
	{
		if (bSkipRecursiveDialogCreation)
		{
			MakeDialogIfNeeded();
			return;
		}

		bSkipRecursiveDialogCreation = true;
		for (FSlowTask* Scope : Context.ScopeStack)
		{
			if (Scope->MakeDialogIfNeeded())
			{
				// Some dialog in the hierarchy wants to be called back
				bSkipRecursiveDialogCreation = false;
			}
		}
	}
}

bool FSlowTask::MakeDialogIfNeeded()
{
	if (bEnabled && !bCreatedDialog && OpenDialogThreshold.IsSet())
	{
		if (static_cast<float>(FPlatformTime::Seconds() - StartTime) < OpenDialogThreshold.GetValue())
		{
			// Let our caller know that we need to be called back
			return true;
		}

		MakeDialog(bDelayedDialogShowCancelButton, bDelayedDialogAllowInPIE);
	}

	return false;
}

void FSlowTask::Initialize()
{
	if (bEnabled)
	{
		Context.ScopeStack.Push(this);
#if MISCTRACE_ENABLED
		if (Context.ScopeStack.Num() <= GSlowTaskMaxTraceRegionDepth)
		{
			RegionId = TRACE_BEGIN_REGION_WITH_ID(!DefaultMessage.IsEmpty() ? *DefaultMessage.ToString() : TEXT("<SlowTask>"));
		}
#endif
	}
}

void FSlowTask::ForceRefresh(FFeedbackContext& Context)
{
	// We force refresh twice to account for r.oneframethreadlag in slate renderer to avoid
	// missing any visual cue when important transition occurs.
	Context.RequestUpdateUI(true);
	Context.RequestUpdateUI(true);
}

void FSlowTask::Destroy()
{
	if (bEnabled)
	{
		if (bCreatedDialog)
		{
			checkSlow(GIsSlowTask);

			// Make sure we see the progress fully updated just before destroying it
			ForceRefresh(Context);

			Context.FinalizeSlowTask();
		}

		FSlowTaskStack& Stack = Context.ScopeStack;
		if (ensure(Stack.Num() != 0))
		{
#if MISCTRACE_ENABLED
			if (Context.ScopeStack.Num() <= GSlowTaskMaxTraceRegionDepth)
			{
				if (RegionId != 0)
				{
					TRACE_END_REGION_WITH_ID(RegionId);
				}
			}
#endif

			FSlowTask* Task = Stack.Last();
			if (!IsEngineExitRequested() && ensureMsgf(Task == this, TEXT("Out-of-order slow task construction/destruction: destroying '%s' but '%s' is at the top of the stack"), *DefaultMessage.ToString(), *Task->DefaultMessage.ToString()))
			{
				Stack.Pop(EAllowShrinking::No);
			}
			else
			{
				Stack.RemoveSingleSwap(this, EAllowShrinking::No);
			}
		}

		if (Stack.Num() != 0)
		{
			// Stop anything else contributing to the parent frame
			FSlowTask* Parent = Stack.Last();
			Parent->EnterProgressFrame(0, Parent->FrameMessage);

			Parent->Context.RequestUpdateUI();
		}
	}
}

void FSlowTask::MakeDialogDelayed(float Threshold, bool bShowCancelButton, bool bAllowInPIE)
{
	OpenDialogThreshold = Threshold;
	bDelayedDialogShowCancelButton = bShowCancelButton;
	bDelayedDialogAllowInPIE = bAllowInPIE;
}

void FSlowTask::EnterProgressFrame(float ExpectedWorkThisFrame, const FText& Text)
{
	if (bEnabled)
	{
		check(IsInGameThread());

		// Should actually be FrameMessage = Text; but this code is to investigate crashes in FSlowTask::GetCurrentMessage()
		if (!Text.IsEmpty())
		{
			FrameMessage = Text;
		}
		else
		{
			FrameMessage = FText::GetEmpty();
		}
		CompletedWork += CurrentFrameScope;

		const float WorkRemaining = TotalAmountOfWork - CompletedWork;
		// Add a small threshold here because when there are a lot of tasks, numerical imprecision can add up and trigger this.
		ensureMsgf(ExpectedWorkThisFrame <= 1.01f * TotalAmountOfWork - CompletedWork, TEXT("Work overflow in slow task. Please revise call-site to account for entire progress range."));
		CurrentFrameScope = FMath::Min(WorkRemaining, ExpectedWorkThisFrame);

		TickProgress();
	}
}

void FSlowTask::TickProgress()
{
	if (bEnabled)
	{
		check(IsInGameThread());

		// Make sure OS events are getting through while the task is being processed
		FPlatformMisc::PumpMessagesForSlowTask();

		MakeRecursiveDialogIfNeeded();
	
		Context.RequestUpdateUI();
	}
}

void FSlowTask::ForceRefresh()
{
	if (bEnabled)
	{
		ForceRefresh(Context);
	}
}

const FText& FSlowTask::GetCurrentMessage() const
{
	return FrameMessage.IsEmpty() ? DefaultMessage : FrameMessage;
}

void FSlowTask::MakeDialog(bool bShowCancelButton, bool bAllowInPIE)
{
	const auto IsDisabledByPIE = [this, bAllowInPIE]() { return Context.IsPlayingInEditor() && !bAllowInPIE; };
	const bool bIsDialogAllowed = bEnabled && IsInGameThread() && !GIsSilent && !IsDisabledByPIE() && !IsRunningCommandlet() && Visibility != ESlowTaskVisibility::Invisible;
	if (bIsDialogAllowed && !GIsSlowTask)
	{
		Context.StartSlowTask(GetCurrentMessage(), bShowCancelButton);
		if (GIsSlowTask)
		{
			bCreatedDialog = true;

			// Refresh UI after dialog has been created
			ForceRefresh(Context);
		}
	}
}

bool FSlowTask::ShouldCancel() const
{
	if (bEnabled && GIsSlowTask)
	{
		check(IsInGameThread()); // FSlowTask is only meant to be used on the main thread currently

		// update the UI from time to time (throttling is done in RequestUpdateUI) so that the cancel button interaction can be detected: 
		Context.RequestUpdateUI();

		return Context.ReceivedUserCancel();
	}
	return false;
}

#if WITH_EDITOR
namespace Private
{
	static void SimulateSlowTask(const TArray<FString>& Arguments)
	{
		double SecondsToStall = 2.0;
		if (Arguments.Num() >= 1)
		{
			LexFromString(SecondsToStall, *Arguments[0]);
		}
		
		SCOPED_NAMED_EVENT_TEXT(TEXT("Simulated SlowTask"), FColor::Red);

		FSlowTask SlowTask(static_cast<float>(SecondsToStall));
		SlowTask.Initialize();
		SlowTask.MakeDialog();

		const double StartTime = FPlatformTime::Seconds();
		while (FPlatformTime::Seconds() - StartTime < SecondsToStall)
		{
			// Busy wait the rest if not slept long enough
			const float SleepTimeSeconds = 0.1f;
			SlowTask.EnterProgressFrame(SleepTimeSeconds);
			FPlatformProcess::SleepNoStats(SleepTimeSeconds);					
		}

		SlowTask.Destroy();	
	}
	
	static FAutoConsoleCommand CmdEditorSimulateSlowTask(
		TEXT("Editor.Debug.SlowTask.Simulate"),
		TEXT("Runs a busy loop for N seconds. Will tick the slow task every 100ms until it is complete"),
		FConsoleCommandWithArgsDelegate::CreateStatic(&SimulateSlowTask)
	);
}
#endif