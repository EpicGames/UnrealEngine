// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/SlowTaskStack.h"
#include "Misc/SlowTask.h"

float FSlowTaskStack::GetProgressFraction(int32 Index) const
{
	const int32 StartIndex = Num() - 1;
	const int32 EndIndex = Index;

	float Progress = 0.f;
	for (int32 CurrentIndex = StartIndex; CurrentIndex >= EndIndex; --CurrentIndex)
	{
		const FSlowTask* Scope = (*this)[CurrentIndex];
		
		const float ThisScopeCompleted = float(Scope->CompletedWork) / Scope->TotalAmountOfWork;
		const float ThisScopeCurrentFrame = float(Scope->CurrentFrameScope) / Scope->TotalAmountOfWork;

		Progress = ThisScopeCompleted + ThisScopeCurrentFrame*Progress;
	}

	return Progress;
}
