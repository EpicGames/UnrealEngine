// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/SourceLocationUtils.h"
#include "Containers/Utf8String.h"
#include "Serialization/CompactBinaryWriter.h"

namespace UE::SourceLocation::Private
{

	/**
	 * Helper to append a formatted full source location to either a string or string builder.
	 *
	 * @param Dest      Destination string or string builder
	 * @param Location  Source location to format
	 */
	template <typename FmtCharType, typename DestBufferType>
	static void AppendFull(DestBufferType& Dest, const FSourceLocation& Location)
	{
	#if UE_INCLUDE_SOURCE_LOCATION
		Dest.Appendf(CHARTEXT(FmtCharType, "%hs(%d:%d) %hs"), Location.GetFileName(), Location.GetLine(), Location.GetColumn(), Location.GetFunctionName());
	#endif // UE_INCLUDE_SOURCE_LOCATION
	}

	/**
	 * Helper to append a formatted file and line source location to either a string or string builder.
	 *
	 * @param Dest      Destination string or string builder
	 * @param Location  Source location to format
	 */
	template <typename FmtCharType, typename DestBufferType>
	static void AppendFileAndLine(DestBufferType& Dest, const FSourceLocation& Location)
	{
	#if UE_INCLUDE_SOURCE_LOCATION
		Dest.Appendf(CHARTEXT(FmtCharType, "%hs(%d)"), Location.GetFileName(), Location.GetLine());
	#endif // UE_INCLUDE_SOURCE_LOCATION
	}

	void SerializeForLogFull(FCbWriter& Writer, const FSourceLocation& Location)
	{
		Writer.BeginObject();
	#if UE_INCLUDE_SOURCE_LOCATION
		Writer.AddString(ANSITEXTVIEW("$type"), ANSITEXTVIEW("SourceLocationFull"));
		Writer.AddString(ANSITEXTVIEW("$text"), WriteToUtf8String<512>(Full(Location)));
		Writer.AddString(ANSITEXTVIEW("File"), Location.GetFileName());
		Writer.AddInteger(ANSITEXTVIEW("Line"), Location.GetLine());
		Writer.AddInteger(ANSITEXTVIEW("Column"), Location.GetColumn());
		Writer.AddString(ANSITEXTVIEW("Function"), Location.GetFunctionName());
	#endif // UE_INCLUDE_SOURCE_LOCATION
		Writer.EndObject();
	}

	void SerializeForLogFileAndLine(FCbWriter& Writer, const FSourceLocation& Location)
	{
		Writer.BeginObject();
	#if UE_INCLUDE_SOURCE_LOCATION
		Writer.AddString(ANSITEXTVIEW("$type"), ANSITEXTVIEW("SourceLocationFileAndLine"));
		Writer.AddString(ANSITEXTVIEW("$text"), WriteToUtf8String<300>(FileAndLine(Location)));
		Writer.AddString(ANSITEXTVIEW("File"), Location.GetFileName());
		Writer.AddInteger(ANSITEXTVIEW("Line"), Location.GetLine());
	#endif // UE_INCLUDE_SOURCE_LOCATION
		Writer.EndObject();
	}

} // namespace UE::SourceLocation::Private

namespace UE::SourceLocation
{

	FString FFullAdapter::ToString() const
	{
		FString Result;
		ToString(Result);
		return Result;
	}

	FUtf8String FFullAdapter::ToUtf8String() const
	{
		FUtf8String Result;
		ToString(Result);
		return Result;
	}

	void FFullAdapter::ToString(FWideString& Out) const
	{
		Out.Reset();
		Private::AppendFull<WIDECHAR>(Out, Location);
	}

	void FFullAdapter::ToString(FUtf8String& Out) const
	{
		Out.Reset();
		Private::AppendFull<UTF8CHAR>(Out, Location);
	}

	void FFullAdapter::AppendString(FWideStringBuilderBase& Out) const
	{
		Private::AppendFull<WIDECHAR>(Out, Location);
	}

	void FFullAdapter::AppendString(FUtf8StringBuilderBase& Out) const
	{
		Private::AppendFull<UTF8CHAR>(Out, Location);
	}

	FString FFileAndLineAdapter::ToString() const
	{
		FString Result;
		ToString(Result);
		return Result;
	}

	FUtf8String FFileAndLineAdapter::ToUtf8String() const
	{
		FUtf8String Result;
		ToString(Result);
		return Result;
	}

	void FFileAndLineAdapter::ToString(FWideString& Out) const
	{
		Out.Reset();
		Private::AppendFileAndLine<WIDECHAR>(Out, Location);
	}

	void FFileAndLineAdapter::ToString(FUtf8String& Out) const
	{
		Out.Reset();
		Private::AppendFileAndLine<UTF8CHAR>(Out, Location);
	}

	void FFileAndLineAdapter::AppendString(FWideStringBuilderBase& Out) const
	{
		Private::AppendFileAndLine<WIDECHAR>(Out, Location);
	}

	void FFileAndLineAdapter::AppendString(FUtf8StringBuilderBase& Out) const
	{
		Private::AppendFileAndLine<UTF8CHAR>(Out, Location);
	}

} // namespace UE::SourceLocation
