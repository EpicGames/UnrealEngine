// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/StringBuilder.h"

#include "HAL/PlatformMath.h"
#include "HAL/UnrealMemory.h"
#include "Misc/CString.h"
#include "Misc/ScopeExit.h"

template <typename C>
TStringBuilderBase<C>::~TStringBuilderBase()
{
	if (bIsDynamic)
	{
		FreeBuffer(Base);
	}
}

template <typename C>
void TStringBuilderBase<C>::Extend(SIZE_T ExtraCapacity)
{
	checkf(!bIsExternal, TEXT("Extending the capacity of an external string builder is not supported. "
		"Size = %" SIZE_T_FMT " Capacity = %" SIZE_T_FMT" ExtraCapacity = %" SIZE_T_FMT "."),
		SIZE_T(CurPos - Base), SIZE_T(End - Base), ExtraCapacity);

	const SIZE_T OldCapacity = End - Base;
	const SIZE_T NewCapacity = FPlatformMath::Max<SIZE_T>(FMath::RoundUpToPowerOfTwo64(OldCapacity + ExtraCapacity), 32);

	C* NewBase = (C*)AllocBuffer(NewCapacity);

	SIZE_T Pos = CurPos - Base;
	if (Pos)
	{
		FMemory::Memcpy(NewBase, Base, Pos * sizeof(C));
	}

	if (bIsDynamic)
	{
		FreeBuffer(Base);
	}

	Base		= NewBase;
	CurPos		= NewBase + Pos;
	End			= NewBase + NewCapacity;
	bIsDynamic	= true;
}

template <typename C>
inline void* TStringBuilderBase<C>::AllocBuffer(SIZE_T CharCount)
{
	return FMemory::Malloc(CharCount * sizeof(C), alignof(C));
}

template <typename C>
inline void TStringBuilderBase<C>::FreeBuffer(void* Buffer)
{
	FMemory::Free(Buffer);
}

template <typename C>
TStringBuilderBase<C>& TStringBuilderBase<C>::AppendV(const C* Fmt, va_list Args)
{
	for (;;)
	{
		const int32 RemainingSize = (int32)(End - CurPos);

		va_list ArgPack;
		va_copy(ArgPack, Args);
		const int32 RequiredSize = TCString<C>::FormatV(CurPos, RemainingSize, Fmt, ArgPack);
		va_end(ArgPack);

		if (RequiredSize < 0)
		{
			UE_DEBUG_BREAK();
			return *this;
		}

		if (RequiredSize < RemainingSize)
		{
			CurPos += RequiredSize;
			return *this;
		}

		// Total size will be rounded up to the next power of two.
		Extend(RequiredSize - RemainingSize + 1);
	}
}

template <typename C>
TStringBuilderBase<C>& TStringBuilderBase<C>::AppendfImpl(BuilderType& Self, const C* Fmt, ...)
{
	va_list ArgPack;
	va_start(ArgPack, Fmt);
	ON_SCOPE_EXIT { va_end(ArgPack); };
	return Self.AppendV(Fmt, ArgPack);
}

// Instantiate templates once

template class TStringBuilderBase<ANSICHAR>;
template class TStringBuilderBase<UTF8CHAR>;
template class TStringBuilderBase<WIDECHAR>;
