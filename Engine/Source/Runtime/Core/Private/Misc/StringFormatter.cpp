// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/StringFormatter.h"

#include "Containers/AnsiString.h"
#include "Containers/Utf8String.h"
#include "Misc/AutomationTest.h"
#include "Misc/ExpressionParser.h"

#define LOCTEXT_NAMESPACE "StringFormatter"

FStringFormatArg::FStringFormatArg(const int32 Value) : Type(Int), IntValue(Value) {}
FStringFormatArg::FStringFormatArg(const uint32 Value) : Type(UInt), UIntValue(Value) {}
FStringFormatArg::FStringFormatArg(const int64 Value) : Type(Int), IntValue(Value) {}
FStringFormatArg::FStringFormatArg(const uint64 Value) : Type(UInt), UIntValue(Value) {}
FStringFormatArg::FStringFormatArg(const float Value) : Type(Double), DoubleValue(Value) {}
FStringFormatArg::FStringFormatArg(const double Value) : Type(Double), DoubleValue(Value) {}
FStringFormatArg::FStringFormatArg(FString Value) : Type(String), StringValue(MoveTemp(Value)) {}
FStringFormatArg::FStringFormatArg(FStringView Value) : Type(String), StringValue(Value) {}
FStringFormatArg::FStringFormatArg(const ANSICHAR* Value) : Type(StringLiteralANSI), StringLiteralANSIValue(Value) {}
FStringFormatArg::FStringFormatArg(const WIDECHAR* Value) : Type(StringLiteralWIDE), StringLiteralWIDEValue(Value) {}
FStringFormatArg::FStringFormatArg(const UCS2CHAR* Value) : Type(StringLiteralUCS2), StringLiteralUCS2Value(Value) {}
FStringFormatArg::FStringFormatArg(const UTF8CHAR* Value) : Type(StringLiteralUTF8), StringLiteralUTF8Value(Value) {}
FStringFormatArg& FStringFormatArg::operator=(const FStringFormatArg& Other)
{
	if (this != &Other)
	{
		Type = Other.Type;
		switch (Type)
		{
			case Int: 				IntValue = Other.IntValue; break;
			case UInt: 				UIntValue = Other.UIntValue; break;
			case Double: 			IntValue = Other.IntValue; break;
			case String: 			StringValue = Other.StringValue; break;
			case StringLiteralANSI: StringLiteralANSIValue = Other.StringLiteralANSIValue; break;
			case StringLiteralWIDE: StringLiteralWIDEValue = Other.StringLiteralWIDEValue; break;
			case StringLiteralUCS2: StringLiteralUCS2Value = Other.StringLiteralUCS2Value; break;
			case StringLiteralUTF8: StringLiteralUTF8Value = Other.StringLiteralUTF8Value; break;
		}
	}
	return *this;
}
FStringFormatArg& FStringFormatArg::operator=(FStringFormatArg&& Other)
{
	if (this != &Other)
	{
		Type = Other.Type;
		switch (Type)
		{
			case Int: 				IntValue = Other.IntValue; break;
			case UInt: 				UIntValue = Other.UIntValue; break;
			case Double: 			IntValue = Other.IntValue; break;
			case String: 			StringValue = MoveTemp(Other.StringValue); break;
			case StringLiteralANSI: StringLiteralANSIValue = Other.StringLiteralANSIValue; break;
			case StringLiteralWIDE: StringLiteralWIDEValue = Other.StringLiteralWIDEValue; break;
			case StringLiteralUCS2: StringLiteralUCS2Value = Other.StringLiteralUCS2Value; break;
			case StringLiteralUTF8: StringLiteralUTF8Value = Other.StringLiteralUTF8Value; break;
		}
	}
	return *this;
}

template <typename StringType>
void AppendToString(const FStringFormatArg& Arg, StringType& StringToAppendTo)
{
	switch(Arg.Type)
	{
		case FStringFormatArg::Int: 				StringToAppendTo.Append(LexToString<StringType>(Arg.IntValue)); break;
		case FStringFormatArg::UInt: 				StringToAppendTo.Append(LexToString<StringType>(Arg.UIntValue)); break;
		case FStringFormatArg::Double: 				StringToAppendTo.Append(LexToString<StringType>(Arg.DoubleValue)); break;
		case FStringFormatArg::String: 				StringToAppendTo.AppendChars(*Arg.StringValue, Arg.StringValue.Len()); break;
		case FStringFormatArg::StringLiteralANSI: 	StringToAppendTo.Append(Arg.StringLiteralANSIValue); break;
		case FStringFormatArg::StringLiteralWIDE: 	StringToAppendTo.Append(Arg.StringLiteralWIDEValue); break;
		case FStringFormatArg::StringLiteralUCS2: 	StringToAppendTo.Append(Arg.StringLiteralUCS2Value); break;
		case FStringFormatArg::StringLiteralUTF8: 	StringToAppendTo.Append(Arg.StringLiteralUTF8Value); break;
	}
}

/** Token representing a literal string inside the string */
template <typename CharType>
struct TStringLiteral
{
	explicit TStringLiteral(const TStringToken<CharType>& InString)
		: String(InString)
		, Len(UE_PTRDIFF_TO_INT32(InString.GetTokenEndPos() - InString.GetTokenStartPos()))
	{
	}

	/** The string literal token */
	TStringToken<CharType> String;
	/** Cached length of the string */
	int32 Len;
};

/** Token representing a user-defined token, such as {Argument} */
template <typename CharType>
struct TTokenFormatSpecifier
{
	explicit TTokenFormatSpecifier(const TStringToken<CharType>& InIdentifier, const TStringToken<CharType>& InEntireToken)
		: Identifier(InIdentifier)
		, EntireToken(InEntireToken)
		, Len(UE_PTRDIFF_TO_INT32(Identifier.GetTokenEndPos() - Identifier.GetTokenStartPos()))
	{
	}

	/** The identifier part of the token */
	TStringToken<CharType> Identifier;
	/** The entire token */
	TStringToken<CharType> EntireToken;
	/** Cached length of the identifier */
	int32 Len;
};

/** Token representing a user-defined index token, such as {0} */
template <typename CharType>
struct TIndexSpecifier
{
	explicit TIndexSpecifier(int32 InIndex, const TStringToken<CharType>& InEntireToken)
		: Index(InIndex)
		, EntireToken(InEntireToken)
	{
	}

	/** The index of the parsed token */
	int32 Index;
	/** The entire token */
	TStringToken<CharType> EntireToken;
};

/** Token representing an escaped character */
template <typename CharType>
struct TEscapedCharacter
{
	explicit TEscapedCharacter(CharType InChar)
		: Character(InChar)
	{
	}

	/** The character that was escaped */
	CharType Character;
};

DEFINE_EXPRESSION_NODE_TYPE(TStringLiteral<ANSICHAR>, 0xB1F8D5E2, 0xE9004121, 0x9C4FEC8B, 0x1B5CFD15)
DEFINE_EXPRESSION_NODE_TYPE(TTokenFormatSpecifier<ANSICHAR>, 0x6E9A920F, 0x713F4E66, 0x9917D2C6, 0xC60076F0)
DEFINE_EXPRESSION_NODE_TYPE(TIndexSpecifier<ANSICHAR>, 0xEFAB3AF9, 0x17FF4EC8, 0x8C207300, 0x2778DC5D)
DEFINE_EXPRESSION_NODE_TYPE(TEscapedCharacter<ANSICHAR>, 0xEAF11B45, 0x3FCF4413, 0x916B2958, 0x93407326)
DEFINE_EXPRESSION_NODE_TYPE(TStringLiteral<WIDECHAR>, 0x03ED3A25, 0x85D94664, 0x8A8001A1, 0xDCC637F7)
DEFINE_EXPRESSION_NODE_TYPE(TTokenFormatSpecifier<WIDECHAR>, 0xAAB48E5B, 0xEDA94853, 0xA951ED2D, 0x0A8E795D)
DEFINE_EXPRESSION_NODE_TYPE(TIndexSpecifier<WIDECHAR>, 0xE11F9937, 0xAF714AC5, 0x88A4E04E, 0x723A753C)
DEFINE_EXPRESSION_NODE_TYPE(TEscapedCharacter<WIDECHAR>, 0x48FF0754, 0x508941BB, 0x9D5447FF, 0xCAC61362)
DEFINE_EXPRESSION_NODE_TYPE(TStringLiteral<UTF8CHAR>, 0xE668FEAA, 0x8B184D67, 0xAF9982EC, 0xDF4B3EA9)
DEFINE_EXPRESSION_NODE_TYPE(TTokenFormatSpecifier<UTF8CHAR>, 0x70BC93BD, 0x6A3E454A, 0x86B9957C, 0xBE104C9A)
DEFINE_EXPRESSION_NODE_TYPE(TIndexSpecifier<UTF8CHAR>, 0x83BCE88A, 0xC26A42FC, 0xBDADAAE9, 0xC9F4A920)
DEFINE_EXPRESSION_NODE_TYPE(TEscapedCharacter<UTF8CHAR>, 0xFD0F11D5, 0xACA94B8F, 0xA5E65642, 0x1A6CED1B)

template <typename CharType>
FExpressionError GenerateErrorMsg(const TStringToken<CharType>& Token)
{
	FFormatOrderedArguments Args;
	Args.Add(FText::FromString(FString(Token.GetTokenEndPos()).Left(10) + TEXT("...")));
	return FExpressionError(FText::Format(LOCTEXT("InvalidTokenDefinition", "Invalid token definition at '{0}'"), Args));
}

template <typename CharType>
TOptional<FExpressionError> ParseIndex(TExpressionTokenConsumer<CharType>& Consumer, bool bEmitErrors)
{
	auto& Stream = Consumer.GetStream();

	TOptional<TStringToken<CharType>> OpeningChar = Stream.ParseSymbol(CHARTEXT(CharType, '{'));
	if (!OpeningChar.IsSet())
	{
		return {};
	}

	TStringToken<CharType>& EntireToken = OpeningChar.GetValue();

	// Optional whitespace
	Stream.ParseToken([](CharType InC) { return TChar<CharType>::IsWhitespace(InC) ? EParseState::Continue : EParseState::StopBefore; }, &EntireToken);

	// The identifier itself
	TOptional<int32> Index;
	Stream.ParseToken([&](CharType InC) {
		if (TChar<CharType>::IsDigit(InC))
		{
			if (!Index.IsSet())
			{
				Index = 0;
			}
			Index.GetValue() *= 10;
			Index.GetValue() += InC - '0';
			return EParseState::Continue;
		}
		return EParseState::StopBefore;
	}, &EntireToken);

	if (!Index.IsSet())
	{
		// Not a valid token
		if (bEmitErrors)
		{
			return GenerateErrorMsg(EntireToken);
		}
		else
		{
			return {};
		}
	}

	// Optional whitespace
	Stream.ParseToken([](CharType InC) { return TChar<CharType>::IsWhitespace(InC) ? EParseState::Continue : EParseState::StopBefore; }, &EntireToken);
	
	if (!Stream.ParseSymbol(CHARTEXT(CharType, '}'), &EntireToken).IsSet())
	{
		// Not a valid token
		if (bEmitErrors)
		{
			return GenerateErrorMsg(EntireToken);
		}
		else
		{
			return {};
		}
	}

	// Add the token to the consumer. This moves the read position in the stream to the end of the token.
	Consumer.Add(EntireToken, TIndexSpecifier<CharType>(Index.GetValue(), EntireToken));
	return {};
}

template <typename CharType>
TOptional<FExpressionError> ParseSpecifier(TExpressionTokenConsumer<CharType>& Consumer, bool bEmitErrors)
{
	auto& Stream = Consumer.GetStream();

	TOptional<TStringToken<CharType>> OpeningChar = Stream.ParseSymbol(CHARTEXT(CharType, '{'));
	if (!OpeningChar.IsSet())
	{
		return {};
	}

	TStringToken<CharType>& EntireToken = OpeningChar.GetValue();

	// Optional whitespace
	Stream.ParseToken([](CharType InC) { return TChar<CharType>::IsWhitespace(InC) ? EParseState::Continue : EParseState::StopBefore; }, &EntireToken);

	// The identifier itself
	TOptional<TStringToken<CharType>> Identifier = Stream.ParseToken([](CharType InC) {
		if (TChar<CharType>::IsWhitespace(InC) || InC == '}')
		{
			return EParseState::StopBefore;
		}
		else if (TChar<CharType>::IsIdentifier(InC))
		{
			return EParseState::Continue;
		}
		else
		{
			return EParseState::Cancel;
		}

	}, &EntireToken);

	if (!Identifier.IsSet())
	{
		// Not a valid token
		// Not a valid token
		if (bEmitErrors)
		{
			return GenerateErrorMsg(EntireToken);
		}
		else
		{
			return {};
		}
	}

	// Optional whitespace
	Stream.ParseToken([](CharType InC) { return TChar<CharType>::IsWhitespace(InC) ? EParseState::Continue : EParseState::StopBefore; }, &EntireToken);

	if (!Stream.ParseSymbol(CHARTEXT(CharType, '}'), &EntireToken).IsSet())
	{
		// Not a valid token
		if (bEmitErrors)
		{
			return GenerateErrorMsg(EntireToken);
		}
		else
		{
			return {};
		}
	}

	// Add the token to the consumer. This moves the read position in the stream to the end of the token.
	Consumer.Add(EntireToken, TTokenFormatSpecifier<CharType>(Identifier.GetValue(), EntireToken));
	return {};
}

/** Parse an escaped character */
template <typename CharType>
TOptional<FExpressionError> ParseEscapedChar(TExpressionTokenConsumer<CharType>& Consumer, bool bEmitErrors)
{
	static const CharType* ValidEscapeChars = CHARTEXT(CharType, "{`");

	TOptional<TStringToken<CharType>> Token = Consumer.GetStream().ParseSymbol(CHARTEXT(CharType, '`'));
	if (!Token.IsSet())
	{
		return {};
	}

	// Accumulate the next character into the token
	TOptional<TStringToken<CharType>> EscapedChar = Consumer.GetStream().ParseSymbol(&Token.GetValue());
	if (!EscapedChar.IsSet())
	{
		return {};
	}

	// Check for a valid escape character
	const CharType Character = *EscapedChar->GetTokenStartPos();
	if (TCString<CharType>::Strchr(ValidEscapeChars, Character))
	{
		// Add the token to the consumer. This moves the read position in the stream to the end of the token.
		Consumer.Add(Token.GetValue(), TEscapedCharacter<CharType>(Character));
		return {};
	}
	else if (bEmitErrors)
	{
		TString<CharType> CharStr;
		CharStr += Character;
		FFormatOrderedArguments Args;
		Args.Add(FText::FromString(CharStr));
		return FExpressionError(FText::Format(LOCTEXT("InvalidEscapeCharacter", "Invalid escape character '{0}'"), Args));
	}
	else
	{
		return {};
	}
}

/** Parse anything until we find an unescaped { */
template <typename CharType>
TOptional<FExpressionError> ParseLiteral(TExpressionTokenConsumer<CharType>& Consumer, bool bEmitErrors)
{
	// Include a leading { character - if it was a valid argument token it would have been picked up by a previous token definition
	bool bFirstChar = true;
	TOptional<TStringToken<CharType>> Token = Consumer.GetStream().ParseToken([&](CharType C){
		if (C == CHARTEXT(CharType, '{') && !bFirstChar)
		{
			return EParseState::StopBefore;
		}
		else if (C == CHARTEXT(CharType, '`'))
		{
			return EParseState::StopBefore;
		}
		else
		{
			bFirstChar = false;
			// Keep consuming
			return EParseState::Continue;
		}
	});

	if (Token.IsSet())
	{
		// Add the token to the consumer. This moves the read position in the stream to the end of the token.
		Consumer.Add(Token.GetValue(), TStringLiteral<CharType>(Token.GetValue()));
	}
	return {};
}

template <typename CharType>
TStringFormatter<CharType>::TStringFormatter()
{
	using namespace ExpressionParser;

	// Token definition logic for named tokens
	NamedDefinitions.DefineToken([](TExpressionTokenConsumer<CharType>& Consumer)			{ return ParseSpecifier(Consumer, false); });
	NamedDefinitions.DefineToken([](TExpressionTokenConsumer<CharType>& Consumer)			{ return ParseEscapedChar(Consumer, false); });
	NamedDefinitions.DefineToken([](TExpressionTokenConsumer<CharType>& Consumer)			{ return ParseLiteral(Consumer, false); });

	// Token definition logic for strict named tokens - will emit errors for any syntax errors
	StrictNamedDefinitions.DefineToken([](TExpressionTokenConsumer<CharType>& Consumer)	{ return ParseSpecifier(Consumer, true); });
	StrictNamedDefinitions.DefineToken([](TExpressionTokenConsumer<CharType>& Consumer)	{ return ParseEscapedChar(Consumer, true); });
	StrictNamedDefinitions.DefineToken([](TExpressionTokenConsumer<CharType>& Consumer)	{ return ParseLiteral(Consumer, true); });

	// Token definition logic for ordered tokens
	OrderedDefinitions.DefineToken([](TExpressionTokenConsumer<CharType>& Consumer)		{ return ParseIndex(Consumer, false); });
	OrderedDefinitions.DefineToken([](TExpressionTokenConsumer<CharType>& Consumer)		{ return ParseEscapedChar(Consumer, false); });
	OrderedDefinitions.DefineToken([](TExpressionTokenConsumer<CharType>& Consumer)		{ return ParseLiteral(Consumer, false); });

	// Token definition logic for strict ordered tokens - will emit errors for any syntax errors
	StrictOrderedDefinitions.DefineToken([](TExpressionTokenConsumer<CharType>& Consumer)	{ return ParseIndex(Consumer, true); });
	OrderedDefinitions      .DefineToken([](TExpressionTokenConsumer<CharType>& Consumer)	{ return ParseEscapedChar(Consumer, true); });
	StrictOrderedDefinitions.DefineToken([](TExpressionTokenConsumer<CharType>& Consumer)	{ return ParseLiteral(Consumer, true); });
}

template <typename CharType>
TValueOrError<TString<CharType>, FExpressionError> TStringFormatter<CharType>::FormatInternal(const CharType* InExpression, const TMap<TString<CharType>, FStringFormatArg>& Args, bool bStrict) const
{
	TValueOrError<TArray<TExpressionToken<CharType>>, FExpressionError> Result = ExpressionParser::Lex(InExpression, bStrict ? StrictNamedDefinitions : NamedDefinitions);
	if (!Result.IsValid())
	{
		return MakeError(Result.StealError());
	}

	TArray<TExpressionToken<CharType>>& Tokens = Result.GetValue();
	if (Tokens.Num() == 0)
	{
		return MakeValue(InExpression);
	}

	// This code deliberately tries to reallocate as little as possible
	TString<CharType> Formatted;
	Formatted.Reserve(UE_PTRDIFF_TO_INT32(Tokens.Last().Context.GetTokenEndPos() - InExpression));
	for (const TExpressionToken<CharType>& Token : Tokens)
	{
		if (const TStringLiteral<CharType>* Literal = Token.Node.template Cast<TStringLiteral<CharType>>())
		{
			Formatted.AppendChars(Literal->String.GetTokenStartPos(), Literal->Len);
		}
		else if (const TEscapedCharacter<CharType>* Escaped = Token.Node.template Cast<TEscapedCharacter<CharType>>())
		{
			Formatted.AppendChar(Escaped->Character);
		}
		else if (const TTokenFormatSpecifier<CharType>* FormatToken = Token.Node.template Cast<TTokenFormatSpecifier<CharType>>())
		{
			const FStringFormatArg* Arg = nullptr;
			for (const TPair<TString<CharType>, FStringFormatArg>& Pair : Args)
			{
				if (Pair.Key.Len() == FormatToken->Len && TCString<CharType>::Strnicmp(FormatToken->Identifier.GetTokenStartPos(), *Pair.Key, FormatToken->Len) == 0)
				{
					Arg = &Pair.Value;
					break;
				}
			}

			if (Arg)
			{
				AppendToString(*Arg, Formatted);
			}
			else if (bStrict)
			{
				return MakeError(FText::Format(LOCTEXT("UndefinedFormatSpecifier", "Undefined format token: {0}"), FText::FromString(FString(FormatToken->Identifier.GetString()))));
			}
			else
			{
				// No replacement found, so just add the original token string
				const int32 Length = UE_PTRDIFF_TO_INT32(FormatToken->EntireToken.GetTokenEndPos() - FormatToken->EntireToken.GetTokenStartPos());
				Formatted.AppendChars(FormatToken->EntireToken.GetTokenStartPos(), Length);
			}
		}
	}

	return MakeValue(MoveTemp(Formatted));
}

template <typename CharType>
TValueOrError<TString<CharType>, FExpressionError> TStringFormatter<CharType>::FormatInternal(const CharType* InExpression, const TArray<FStringFormatArg>& Args, bool bStrict) const
{
	TValueOrError<TArray<TExpressionToken<CharType>>, FExpressionError> Result = ExpressionParser::Lex(InExpression, bStrict ? StrictOrderedDefinitions : OrderedDefinitions);
	if (!Result.IsValid())
	{
		return MakeError(Result.StealError());
	}

	TArray<TExpressionToken<CharType>>& Tokens = Result.GetValue();
	if (Tokens.Num() == 0)
	{
		return MakeValue(InExpression);
	}
	
	// This code deliberately tries to reallocate as little as possible
	TString<CharType> Formatted;
	Formatted.Reserve(UE_PTRDIFF_TO_INT32(Tokens.Last().Context.GetTokenEndPos() - InExpression));
	for (const TExpressionToken<CharType>& Token : Tokens)
	{
		if (const TStringLiteral<CharType>* Literal = Token.Node.template Cast<TStringLiteral<CharType>>())
		{
			Formatted.AppendChars(Literal->String.GetTokenStartPos(), Literal->Len);
		}
		else if (const TEscapedCharacter<CharType>* Escaped = Token.Node.template Cast<TEscapedCharacter<CharType>>())
		{
			Formatted.AppendChar(Escaped->Character);
		}
		else if (const TIndexSpecifier<CharType>* IndexToken = Token.Node.template Cast<TIndexSpecifier<CharType>>())
		{
			if (Args.IsValidIndex(IndexToken->Index))
			{
				AppendToString(Args[IndexToken->Index], Formatted);
			}
			else if (bStrict)
			{
				return MakeError(FText::Format(LOCTEXT("InvalidArgumentIndex", "Invalid argument index: {0}"), FText::AsNumber(IndexToken->Index)));
			}
			else
			{
				// No replacement found, so just add the original token string
				const int32 Length = UE_PTRDIFF_TO_INT32(IndexToken->EntireToken.GetTokenEndPos() - IndexToken->EntireToken.GetTokenStartPos());
				Formatted.AppendChars(IndexToken->EntireToken.GetTokenStartPos(), Length);
			}
		}
	}

	return MakeValue(MoveTemp(Formatted));
}

/** Default formatter for string formatting - thread safe since all formatting is const */
template <typename CharType>
TStringFormatter<CharType>& GetDefaultFormatter()
{
	static TStringFormatter<CharType> DefaultFormatter;
	return DefaultFormatter;
}

FAnsiString FAnsiString::FormatImpl(const ANSICHAR* InFormatString, const FAnsiStringFormatNamedArguments& InNamedArguments)
{
	TStringFormatter<ANSICHAR>& DefaultFormatter = GetDefaultFormatter<ANSICHAR>();
	return DefaultFormatter.Format(InFormatString, InNamedArguments);
}

FAnsiString FAnsiString::FormatImpl(const ANSICHAR* InFormatString, const FStringFormatOrderedArguments& InOrderedArguments)
{
	TStringFormatter<ANSICHAR>& DefaultFormatter = GetDefaultFormatter<ANSICHAR>();
	return DefaultFormatter.Format(InFormatString, InOrderedArguments);
}

FString FString::FormatImpl(const TCHAR* InFormatString, const FStringFormatNamedArguments& InNamedArguments)
{
	TStringFormatter<TCHAR>& DefaultFormatter = GetDefaultFormatter<TCHAR>();
	return DefaultFormatter.Format(InFormatString, InNamedArguments);
}

FString FString::FormatImpl(const TCHAR* InFormatString, const FStringFormatOrderedArguments& InOrderedArguments)
{
	TStringFormatter<TCHAR>& DefaultFormatter = GetDefaultFormatter<TCHAR>();
	return DefaultFormatter.Format(InFormatString, InOrderedArguments);
}

FUtf8String FUtf8String::FormatImpl(const UTF8CHAR* InFormatString, const FUtf8StringFormatNamedArguments& InNamedArguments)
{
	TStringFormatter<UTF8CHAR>& DefaultFormatter = GetDefaultFormatter<UTF8CHAR>();
	return DefaultFormatter.Format(InFormatString, InNamedArguments);
}

FUtf8String FUtf8String::FormatImpl(const UTF8CHAR* InFormatString, const FStringFormatOrderedArguments& InOrderedArguments)
{
	TStringFormatter<UTF8CHAR>& DefaultFormatter = GetDefaultFormatter<UTF8CHAR>();
	return DefaultFormatter.Format(InFormatString, InOrderedArguments);
}

template class TStringFormatter<ANSICHAR>;
template class TStringFormatter<UTF8CHAR>;
template class TStringFormatter<WIDECHAR>;

#undef LOCTEXT_NAMESPACE
