// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/StringUtility.h"

#include "CoreTypes.h"
#include "Logging/LogMacros.h"
#include "Misc/CString.h"


DEFINE_LOG_CATEGORY_STATIC(LogStringUtility, Log, All);

FString StringUtility::UnescapeURI(const FString& URLString)
{
	// @todo: support decoding of all escaped symbols in a better way

	FString DecodedString = URLString.Replace(TEXT("%20"), TEXT(" "), ESearchCase::CaseSensitive);
	DecodedString = DecodedString.Replace(TEXT("%21"), TEXT("!"), ESearchCase::CaseSensitive);
	DecodedString = DecodedString.Replace(TEXT("%3F"), TEXT("?"), ESearchCase::CaseSensitive);

	return DecodedString;
}
