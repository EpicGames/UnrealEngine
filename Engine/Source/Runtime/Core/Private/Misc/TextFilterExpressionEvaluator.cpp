// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/TextFilterExpressionEvaluator.h"

#include "Containers/SparseArray.h"
#include "Math/BasicMathExpressionEvaluator.h"
#include "Misc/CString.h"
#include "Misc/ExpressionParserTypes.h"
#include "Misc/Guid.h"
#include "Templates/Function.h"
#include "Templates/ValueOrError.h"

namespace TextFilterExpressionParser
{
	/**
	 * This contains all the symbols that can define breaking points between text and an operator
	 * Note: We don't include + and - in this list as these are valid to use inside text and numbers, and should be consumed as part of the text token
	 */
	static const TCHAR BasicTextBreakingCharacters[]	= { TEXT('('), TEXT(')'), TEXT('!'), TEXT('&'), TEXT('|'), TEXT(' ') };						// ETextFilterExpressionEvaluatorMode::BasicString
	static const TCHAR ComplexTextBreakingCharacters[]	= { TEXT('('), TEXT(')'), TEXT('='), TEXT(':'), TEXT('<'), TEXT('>'), TEXT('!'), TEXT('&'), TEXT('|'), TEXT(' ') };	// ETextFilterExpressionEvaluatorMode::Complex

	const TCHAR* FSubExpressionStart::Monikers[]		= { TEXT("(") };
	const TCHAR* FSubExpressionEnd::Monikers[]			= { TEXT(")") };

	const TCHAR* FEqual::Monikers[]						= { TEXT("=="), TEXT("="), TEXT(":") };
	const TCHAR* FNotEqual::Monikers[]					= { TEXT("!="), TEXT("!:") };
	const TCHAR* FLess::Monikers[]						= { TEXT("<") };
	const TCHAR* FLessOrEqual::Monikers[]				= { TEXT("<="), TEXT("<:") };
	const TCHAR* FGreater::Monikers[]					= { TEXT(">") };
	const TCHAR* FGreaterOrEqual::Monikers[]			= { TEXT(">="), TEXT(">:") };

	const TCHAR* FOr::Monikers[]						= { TEXT("OR "), TEXT("OR("), TEXT("||"), TEXT("|") };
	const TCHAR* FAnd::Monikers[]						= { TEXT("AND "), TEXT("AND("), TEXT("&&"), TEXT("&") };
	const TCHAR* FNot::Monikers[]						= { TEXT("NOT "), TEXT("NOT("), TEXT("!") };

	/** Consume an operator from the specified consumer's stream, if one exists at the current read position */
	template<typename TSymbol>
	TOptional<FExpressionError> ConsumeOperator(FExpressionTokenConsumer& Consumer)
	{
		auto& Stream = Consumer.GetStream();

		for (const TCHAR* Moniker : TSymbol::Monikers)
		{
			TOptional<FStringToken> OperatorToken = Stream.ParseToken(Moniker);
			if (OperatorToken.IsSet())
			{
				Consumer.Add(OperatorToken.GetValue(), TSymbol());
			}
		}

		return TOptional<FExpressionError>();
	}

	/** Consume a number from the specified consumer's stream, if one exists at the current read position */
	TOptional<FExpressionError> ConsumeNumber(FExpressionTokenConsumer& Consumer)
	{
		auto& Stream = Consumer.GetStream();

		TOptional<FStringToken> NumberToken = ExpressionParser::ParseLocalizedNumberWithAgnosticFallback(Stream);
		
		if (NumberToken.IsSet())
		{
			Consumer.Add(NumberToken.GetValue(), FExpressionNode(FTextToken(NumberToken.GetValue().GetString(), ETextFilterTextComparisonMode::Partial, FTextToken::EInvertResult::No)));
		}

		return TOptional<FExpressionError>();
	}

	/** Transform the given string to remove any escape character sequences found in a quoted string */
	void UnescapeQuotedString(FString& Str, const TCHAR InQuoteChar)
	{
		const TCHAR EscapedQuote[] = { TEXT('\\'), InQuoteChar, TEXT('\0') };
		const TCHAR UnescapedQuote[] = { InQuoteChar, TEXT('\0') };

		// Unescape any literal quotes within the string
		Str.ReplaceInline(EscapedQuote, UnescapedQuote);
		Str.ReplaceInline(TEXT("\\\\"), TEXT("\\"));
	}

	/** Given a potential string, we produce a final FTextToken for it using the correct comparison mode (as inferred from the given string) */
	FTextToken CreateTextTokenFromUnquotedString(FString InString)
	{
		ETextFilterTextComparisonMode TextComparisonMode = ETextFilterTextComparisonMode::Partial;

		// The final string may embed the TextCmpExact (+) operator, or the TextCmpAnchor operators (...) - test for those now so we can use the correct comparison mode
		if (InString.Len() > 0 && InString[0] == '+')
		{
			// Matched TextCmpExact - update the comparison mode and remove the + token from the start of the string
			TextComparisonMode = ETextFilterTextComparisonMode::Exact;
			InString.RemoveAt(0, EAllowShrinking::No);
		}
		else if (InString.Len() > 2 && InString.StartsWith(TEXT("..."), ESearchCase::CaseSensitive))
		{
			// Matched TextCmpAnchor (pre-unary) - update the comparison mode and remove the ... token from the start of the string
			TextComparisonMode = ETextFilterTextComparisonMode::EndsWith;
			InString.RemoveAt(0, 3, EAllowShrinking::No);
		}
		else if (InString.Len() > 2 && InString.EndsWith(TEXT("..."), ESearchCase::CaseSensitive))
		{
			// Matched TextCmpAnchor (post-unary) - update the comparison mode and remove the ... token from the end of the string
			TextComparisonMode = ETextFilterTextComparisonMode::StartsWith;
			InString.RemoveAt(InString.Len() - 3, 3, EAllowShrinking::No);
		}

		// To preserve behavior with the old text filter, the final string may also contain a TextCmpInvert (-) operator (after stripping the TextCmpExact or TextCmpAnchor tokens from the start)
		FTextToken::EInvertResult InvertResult = FTextToken::EInvertResult::No;
		if (InString.Len() > 0 && InString[0] == '-')
		{
			// Matched TextCmpInvert - remove the - token from the start of the string
			InvertResult = FTextToken::EInvertResult::Yes;
			InString.RemoveAt(0, EAllowShrinking::No);
		}

		// Finally, if our string starts and ends with a quote, we need to strip those off now
		if (InString.Len() > 1 && (InString[0] == '"' || InString[0] == '\''))
		{
			const TCHAR QuoteChar = InString[0];
			if (InString[InString.Len() - 1] == QuoteChar)
			{
				// Remove the quotes
				InString.RemoveAt(0, EAllowShrinking::No);
				InString.RemoveAt(InString.Len() - 1, EAllowShrinking::No);
			}
		}

		return FTextToken(MoveTemp(InString), TextComparisonMode, InvertResult);
	}

	/** Consume the text from the specified consumer's stream */
	TOptional<FExpressionError> ConsumeTextImpl(FExpressionTokenConsumer& Consumer, const TFunctionRef<bool(TCHAR)> IsBreakingCharacter)
	{
		auto& Stream = Consumer.GetStream();

		FString FinalString;
		FString CurrentQuotedString;

		TCHAR QuoteChar = TEXT('\0');
		int32 NumConsecutiveSlashes = 0;
		TOptional<FStringToken> TextToken = Stream.ParseToken([&](TCHAR InC)
		{
			if (QuoteChar == 0) // Parsing a non-quoted string...
			{
				// Are we starting a quoted sub-string?
				if (InC == '"' || InC == '\'')
				{
					CurrentQuotedString.AppendChar(InC);
					QuoteChar = InC;
					NumConsecutiveSlashes = 0;
				}
				else
				{
					// Consume until we hit a breaking character
					if (IsBreakingCharacter(InC))
					{
						return EParseState::StopBefore;
					}

					FinalString.AppendChar(InC);
				}
			}
			else // Parsing a quoted sub-string...
			{
				CurrentQuotedString.AppendChar(InC);

				// Are we ending a quoted sub-string?
				if (InC == QuoteChar && NumConsecutiveSlashes%2 == 0)
				{
					UnescapeQuotedString(CurrentQuotedString, QuoteChar);
					FinalString.Append(CurrentQuotedString);

					CurrentQuotedString.Reset();
					QuoteChar = TEXT('\0');
				}

				if (InC == '\\')
				{
					NumConsecutiveSlashes++;
				}
				else
				{
					NumConsecutiveSlashes = 0;
				}
			}

			return EParseState::Continue;
		});

		if (TextToken.IsSet())
		{
			Consumer.Add(TextToken.GetValue(), CreateTextTokenFromUnquotedString(MoveTemp(FinalString)));
		}

		return TOptional<FExpressionError>();
	}

	/** Consume the basic text from the specified consumer's stream */
	TOptional<FExpressionError> ConsumeBasicText(FExpressionTokenConsumer& Consumer)
	{
		return ConsumeTextImpl(Consumer, [](TCHAR InC) -> bool
		{
			for (const TCHAR BreakingCharacter : BasicTextBreakingCharacters)
			{
				if (InC == BreakingCharacter)
				{
					return true;
				}
			}
			return false;
		});
	}

	/** Consume the basic text from the specified consumer's stream */
	TOptional<FExpressionError> ConsumeComplexText(FExpressionTokenConsumer& Consumer)
	{
		return ConsumeTextImpl(Consumer, [](TCHAR InC) -> bool
		{
			for (const TCHAR BreakingCharacter : ComplexTextBreakingCharacters)
			{
				if (InC == BreakingCharacter)
				{
					return true;
				}
			}
			return false;
		});
	}
}

#undef DEFINE_TEXT_EXPRESSION_OPERATOR_NODE

/** Dummy context used for testing that the compiled expression is syntactically valid */
class FDummyTextFilterExpressionContext : public ITextFilterExpressionContext
{
public:
	virtual bool TestBasicStringExpression(const FTextFilterString& InValue, const ETextFilterTextComparisonMode InTextComparisonMode) const override
	{
		return false;
	}
	virtual bool TestComplexExpression(const FName& InKey, const FTextFilterString& InValue, const ETextFilterComparisonOperation InComparisonOperation, const ETextFilterTextComparisonMode InTextComparisonMode) const override
	{
		return false;
	}
};

FTextFilterExpressionEvaluator::FTextFilterExpressionEvaluator(const ETextFilterExpressionEvaluatorMode InMode)
	: ExpressionEvaluatorMode(InMode)
	, FilterType(ETextFilterExpressionType::Empty)
{
	ConstructExpressionParser();
}

FTextFilterExpressionEvaluator::FTextFilterExpressionEvaluator(const FTextFilterExpressionEvaluator& Other)
	: ExpressionEvaluatorMode(Other.ExpressionEvaluatorMode)
	, FilterType(ETextFilterExpressionType::Empty)
{
	ConstructExpressionParser();
	SetFilterText(Other.FilterText);
}

FTextFilterExpressionEvaluator::~FTextFilterExpressionEvaluator() = default;

FTextFilterExpressionEvaluator& FTextFilterExpressionEvaluator::operator=(const FTextFilterExpressionEvaluator& Other)
{
	FilterType = ETextFilterExpressionType::Empty;
	FilterText = FText::GetEmpty();
	FilterErrorText = FText::GetEmpty();
	ExpressionTokens.Reset();
	CompiledFilter.Reset();

	// If our source object has a different evaluation mode, we need to re-construct the expression parser
	if (ExpressionEvaluatorMode != Other.ExpressionEvaluatorMode)
	{
		ExpressionEvaluatorMode = Other.ExpressionEvaluatorMode;

		TokenDefinitions = FTokenDefinitions();
		Grammar = FExpressionGrammar();
		JumpTable = TOperatorJumpTable<ITextFilterExpressionContext>();

		ConstructExpressionParser();
	}

	// Re-compile the filter from the other object
	SetFilterText(Other.FilterText);

	return *this;
}

ETextFilterExpressionType FTextFilterExpressionEvaluator::GetFilterType() const
{
	return FilterType;
}

FText FTextFilterExpressionEvaluator::GetFilterText() const
{
	return FilterText;
}

bool FTextFilterExpressionEvaluator::SetFilterText(const FText& InFilterText)
{
	using namespace TextFilterExpressionParser;
	using TextFilterExpressionParser::FTextToken;

	// Has anything changed? (need to test case as the operators are case-sensitive)
	if (FilterText.ToString().Equals(InFilterText.ToString(), ESearchCase::CaseSensitive))
	{
		return false;
	}

	FilterType = ETextFilterExpressionType::Invalid;
	FilterText = InFilterText;
	ExpressionTokens.Reset();
	CompiledFilterSingleBasicStringTextToken.Reset();

	if (FilterText.IsEmptyOrWhitespace())
	{
		// If the filter is empty, just clear out the compiled filter so we match everything
		FilterType = ETextFilterExpressionType::Empty;
		FilterErrorText = FText::GetEmpty();
		CompiledFilter.Reset();
	}
	else
	{
		// Otherwise, re-parse the filter text into a compiled expression
		auto LexResult = ExpressionParser::Lex(*FilterText.ToString(), TokenDefinitions);
		if (LexResult.IsValid())
		{
			auto& TmpLexTokens = LexResult.GetValue();
			ExpressionTokens.Reserve(TmpLexTokens.Num());

			bool bIsComplexExpression = false;

			// If our expression doesn't contain any text tokens, then it can't possibly match anything (this can easily happen when performing simple searches for strings that only contain operators)
			// In this case, we should just treat the entire expression as a single text token
			bool bExpressionContainsText = false;
			for (FExpressionToken& CurrentToken : TmpLexTokens)
			{
				if (CurrentToken.Node.Cast<FTextToken>())
				{
					bExpressionContainsText = true;
					break;
				}
			}

			if (bExpressionContainsText)
			{
				// We inject AND operators between any adjacent text and sub-expression tokens to mimic the old search filter behavior
				bool bWasPreviousTokenValidToInjectANDOperatorAfter = false;
				for (int32 TokenIdx = 0; TokenIdx < TmpLexTokens.Num(); ++TokenIdx)
				{
					FExpressionToken& CurrentToken = TmpLexTokens[TokenIdx];

					if (TokenIdx + 3 < TmpLexTokens.Num())
					{
						if (const FTextToken* TextToken = CurrentToken.Node.Cast<FTextToken>())
						{
							FStringToken FunctionName(CurrentToken.Context);

							FExpressionToken& ExpressionStartToken = TmpLexTokens[TokenIdx + 1];
							if (ExpressionStartToken.Node.Cast<FSubExpressionStart>())
							{
								// We are inside a function, we need the token after the Start, but the TokenIdx is still on the function name
								TokenIdx += 1;
								FExpressionToken* NextToken = &TmpLexTokens[++TokenIdx];
								FStringToken FunctionParameter(NextToken->Context);
								int32 NestedCount = 1;
								for (; TokenIdx < TmpLexTokens.Num(); ++TokenIdx) 
								{
									NextToken = &TmpLexTokens[TokenIdx];

									if (NextToken->Node.Cast<FSubExpressionStart>())
									{
										++NestedCount;
									}
									else if (NextToken->Node.Cast<FSubExpressionEnd>())
									{
										--NestedCount;
										if (NestedCount == 0)
										{
											break;
										}
									}
									FunctionParameter.Accumulate(NextToken->Context);
								}
								FStringToken EmptyToken;
								ExpressionTokens.Add(FExpressionToken(EmptyToken, FSubExpressionStart()));
								ExpressionTokens.Add(FExpressionToken(FunctionName, FTextToken(FunctionName.GetString(), ETextFilterTextComparisonMode::Partial, FTextToken::EInvertResult::No)));
								ExpressionTokens.Add(FExpressionToken(EmptyToken, FFunction()));
								ExpressionTokens.Add(FExpressionToken(FunctionParameter, FTextToken(FunctionParameter.GetString(), ETextFilterTextComparisonMode::Partial, FTextToken::EInvertResult::No)));
								ExpressionTokens.Add(FExpressionToken(EmptyToken, FSubExpressionEnd()));
								continue;
							}
						}
					}
					// Key->value pairs require some form of comparison operator, so if we've found one of those, consider this to be a complex expression
					if (!bIsComplexExpression && 
						(CurrentToken.Node.Cast<FEqual>() || CurrentToken.Node.Cast<FNotEqual>() || CurrentToken.Node.Cast<FLess>() || CurrentToken.Node.Cast<FLessOrEqual>() || CurrentToken.Node.Cast<FGreater>() || CurrentToken.Node.Cast<FGreaterOrEqual>())
						)
					{
						bIsComplexExpression = true;
					}
					
					// Are we in a position to potentially inject a new AND token?
					if (bWasPreviousTokenValidToInjectANDOperatorAfter)
					{
						// We can inject an AND token before any text token, a sub-expression start token, or a pre-unary NOT token
						const bool bIsCurrentTokenValidToInjectANDOperatorBefore = (CurrentToken.Node.Cast<FTextToken>() || CurrentToken.Node.Cast<FSubExpressionStart>() || CurrentToken.Node.Cast<FNot>());
						if (bIsCurrentTokenValidToInjectANDOperatorBefore)
						{
							// Inject the new token before we move the current token into the final list
							ExpressionTokens.Add(FExpressionToken(CurrentToken.Context, FAnd()));
						}
					}

					// Update this for the next run through the loop - we can inject an AND token after any text token, or a sub-expression end token
					bWasPreviousTokenValidToInjectANDOperatorAfter = (CurrentToken.Node.Cast<FTextToken>() || CurrentToken.Node.Cast<FSubExpressionEnd>());

					// Move this token into the final list since we should have injected the AND operator (if required) by now
					ExpressionTokens.Add(MoveTemp(CurrentToken));
				}
			}
			else if (TmpLexTokens.Num() > 0)
			{
				FStringToken CombinedTokenContext = TmpLexTokens[0].Context;
				for (int32 TokenIndex = 1; TokenIndex < TmpLexTokens.Num(); ++TokenIndex)
				{
					CombinedTokenContext.Accumulate(TmpLexTokens[TokenIndex].Context);
				}
				ExpressionTokens.Add(FExpressionToken(CombinedTokenContext, CreateTextTokenFromUnquotedString(FText::TrimPrecedingAndTrailing(FilterText).ToString())));
			}

			{
				TArray<FExpressionToken> LocalExpressionTokens;
				LocalExpressionTokens.Reserve(ExpressionTokens.Num());
				for (const FExpressionToken& Token : ExpressionTokens)
				{
					LocalExpressionTokens.Add(FExpressionToken(Token.Context, Token.Node.Copy()));
				}

				CompiledFilter = ExpressionParser::Compile(MoveTemp(LocalExpressionTokens), Grammar);
			}

			auto& CompiledResult = CompiledFilter.GetValue();
			if (CompiledResult.IsValid())
			{
				FilterType = (bIsComplexExpression) ? ETextFilterExpressionType::Complex : ETextFilterExpressionType::BasicString;
				FilterErrorText = FText::GetEmpty();

				// Evaluate the compiled filter with a dummy context - this will let us know whether it's syntactically valid
				FDummyTextFilterExpressionContext DummyContext;
				EvaluateCompiledExpression(CompiledFilter.GetValue(), DummyContext, &FilterErrorText);

				// If the search string contains no special tokens, optimize by avoiding need to evaluate expression for each item iterated during search
				if (CompiledFilter->GetValue().Num() == 1 && CompiledFilter->GetValue()[0].Type == FCompiledToken::Operand)
				{
					FExpressionResult ExpressionResult = MakeValue(CompiledFilter->GetValue()[0].Node.Copy());
					if (const FTextToken* TextResult = ExpressionResult.GetValue().Cast<FTextToken>())
					{
						CompiledFilterSingleBasicStringTextToken = *TextResult;
					}
				}
			}
			else
			{
				FilterErrorText = CompiledResult.GetError().Text;
			}
		}
		else
		{
			FilterErrorText = LexResult.GetError().Text;
		}
	}

	return true;
}

FText FTextFilterExpressionEvaluator::GetFilterErrorText() const
{
	return FilterErrorText;
}

const TArray<FExpressionToken>& FTextFilterExpressionEvaluator::GetFilterExpressionTokens() const
{
	return ExpressionTokens;
}

bool FTextFilterExpressionEvaluator::TestTextFilter(const ITextFilterExpressionContext& InContext) const
{
	if (FilterType == ETextFilterExpressionType::Empty)
	{
		return true;
	}

	if (CompiledFilter.IsSet())
	{
		return EvaluateCompiledExpression(CompiledFilter.GetValue(), InContext, nullptr);
	}

	return false;
}

void FTextFilterExpressionEvaluator::AddFunctionTokenCallback(FString InFunctionName, FTokenFunctionHandler InCallback)
{
	TokenFunctionHandlers.Add(InFunctionName, InCallback);
}

void FTextFilterExpressionEvaluator::SetupGrammar()
{
	using namespace TextFilterExpressionParser;
	using TextFilterExpressionParser::FTextToken;

	TokenDefinitions.IgnoreWhitespace();
	TokenDefinitions.DefineToken(&ConsumeOperator<FSubExpressionStart>);
	TokenDefinitions.DefineToken(&ConsumeOperator<FSubExpressionEnd>);
	if (ExpressionEvaluatorMode == ETextFilterExpressionEvaluatorMode::Complex)
	{
		// Comparisons are only available for complex expressions
		TokenDefinitions.DefineToken(&ConsumeOperator<FLessOrEqual>);
		TokenDefinitions.DefineToken(&ConsumeOperator<FLess>);
		TokenDefinitions.DefineToken(&ConsumeOperator<FGreaterOrEqual>);
		TokenDefinitions.DefineToken(&ConsumeOperator<FGreater>);
		TokenDefinitions.DefineToken(&ConsumeOperator<FNotEqual>);
		TokenDefinitions.DefineToken(&ConsumeOperator<FEqual>);
	}
	TokenDefinitions.DefineToken(&ConsumeOperator<FOr>);
	TokenDefinitions.DefineToken(&ConsumeOperator<FAnd>);
	TokenDefinitions.DefineToken(&ConsumeOperator<FNot>);
	if (ExpressionEvaluatorMode == ETextFilterExpressionEvaluatorMode::Complex)
	{
		TokenDefinitions.DefineToken(&ConsumeNumber);
	}
	TokenDefinitions.DefineToken((ExpressionEvaluatorMode == ETextFilterExpressionEvaluatorMode::Complex) ? &ConsumeComplexText : &ConsumeBasicText);

	Grammar.DefineGrouping<FSubExpressionStart, FSubExpressionEnd>();
	Grammar.DefineBinaryOperator<FLessOrEqual>(1);
	Grammar.DefineBinaryOperator<FLess>(1);
	Grammar.DefineBinaryOperator<FGreaterOrEqual>(1);
	Grammar.DefineBinaryOperator<FGreater>(1);
	Grammar.DefineBinaryOperator<FNotEqual>(1);
	Grammar.DefineBinaryOperator<FEqual>(1);
	Grammar.DefineBinaryOperator<FOr>(2);
	Grammar.DefineBinaryOperator<FAnd>(2);
	Grammar.DefineBinaryOperator<FFunction>(1);
	Grammar.DefinePreUnaryOperator<FNot>();
}

void FTextFilterExpressionEvaluator::ConstructExpressionParser()
{
	using namespace TextFilterExpressionParser;
	using TextFilterExpressionParser::FTextToken;
	SetupGrammar();

	JumpTable.MapBinary<FLessOrEqual>([](const FTextToken& A, const FTextToken& B, const ITextFilterExpressionContext* InContext)		{ return B.EvaluateAsComplexExpression(InContext, A.GetString(), ETextFilterComparisonOperation::LessOrEqual); });
	JumpTable.MapBinary<FLess>([](const FTextToken& A, const FTextToken& B, const ITextFilterExpressionContext* InContext)				{ return B.EvaluateAsComplexExpression(InContext, A.GetString(), ETextFilterComparisonOperation::Less); });
	JumpTable.MapBinary<FGreaterOrEqual>([](const FTextToken& A, const FTextToken& B, const ITextFilterExpressionContext* InContext)	{ return B.EvaluateAsComplexExpression(InContext, A.GetString(), ETextFilterComparisonOperation::GreaterOrEqual); });
	JumpTable.MapBinary<FGreater>([](const FTextToken& A, const FTextToken& B, const ITextFilterExpressionContext* InContext)			{ return B.EvaluateAsComplexExpression(InContext, A.GetString(), ETextFilterComparisonOperation::Greater); });
	JumpTable.MapBinary<FNotEqual>([](const FTextToken& A, const FTextToken& B, const ITextFilterExpressionContext* InContext)			{ return B.EvaluateAsComplexExpression(InContext, A.GetString(), ETextFilterComparisonOperation::NotEqual); });
	JumpTable.MapBinary<FEqual>([](const FTextToken& A, const FTextToken& B, const ITextFilterExpressionContext* InContext)				{ return B.EvaluateAsComplexExpression(InContext, A.GetString(), ETextFilterComparisonOperation::Equal); });

	JumpTable.MapBinary<FOr>([](const FTextToken& A, const FTextToken& B, const ITextFilterExpressionContext* InContext)				{ return A.EvaluateAsBasicStringExpression(InContext) || B.EvaluateAsBasicStringExpression(InContext); });
	JumpTable.MapBinary<FOr>([](const FTextToken& A, bool B, const ITextFilterExpressionContext* InContext)								{ return A.EvaluateAsBasicStringExpression(InContext) || B; });
	JumpTable.MapBinary<FOr>([](bool A, const FTextToken& B, const ITextFilterExpressionContext* InContext)								{ return A || B.EvaluateAsBasicStringExpression(InContext); });
	JumpTable.MapBinary<FOr>([](bool A, bool B, const ITextFilterExpressionContext* InContext)											{ return A || B; });

	JumpTable.MapBinary<FAnd>([](const FTextToken& A, const FTextToken& B, const ITextFilterExpressionContext* InContext)				{ return A.EvaluateAsBasicStringExpression(InContext) && B.EvaluateAsBasicStringExpression(InContext); });
	JumpTable.MapBinary<FAnd>([](const FTextToken& A, bool B, const ITextFilterExpressionContext* InContext)							{ return A.EvaluateAsBasicStringExpression(InContext) && B; });
	JumpTable.MapBinary<FAnd>([](bool A, const FTextToken& B, const ITextFilterExpressionContext* InContext)							{ return A && B.EvaluateAsBasicStringExpression(InContext); });
	JumpTable.MapBinary<FAnd>([](bool A, bool B, const ITextFilterExpressionContext* InContext)											{ return A && B; });

	JumpTable.MapPreUnary<FNot>([](const FTextToken& V, const ITextFilterExpressionContext* InContext)									{ return !V.EvaluateAsBasicStringExpression(InContext); });
	JumpTable.MapPreUnary<FNot>([](bool V, const ITextFilterExpressionContext* InContext)												{ return !V; });

	JumpTable.MapBinary<FFunction>(
		[this](const TextFilterExpressionParser::FTextToken& A, const TextFilterExpressionParser::FTextToken& B, const ITextFilterExpressionContext* InContext)	
	{
		bool bResult = false;
		if (FTokenFunctionHandler* FunctionCallback = TokenFunctionHandlers.Find(A.GetString().AsString()))
		{
			bResult = FunctionCallback->Execute(B.GetString());
		}
		return bResult;
	});
}

bool FTextFilterExpressionEvaluator::EvaluateCompiledExpression(const ExpressionParser::CompileResultType& InCompiledResult, const ITextFilterExpressionContext& InContext, FText* OutErrorText) const
{
	using namespace TextFilterExpressionParser;
	using TextFilterExpressionParser::FTextToken;

	if (InCompiledResult.IsValid())
	{
		// Optimize by avoiding expression recompile when context has no effect on the compiled result
		if (CompiledFilterSingleBasicStringTextToken.IsSet())
		{
			return CompiledFilterSingleBasicStringTextToken->EvaluateAsBasicStringExpression(&InContext);
		}

		auto EvalResult = ExpressionParser::Evaluate(InCompiledResult.GetValue(), JumpTable, &InContext);
		if (EvalResult.IsValid())
		{
			if (const bool* BoolResult = EvalResult.GetValue().Cast<bool>())
			{
				return *BoolResult;
			}
			else if (const FTextToken* TextResult = EvalResult.GetValue().Cast<FTextToken>())
			{
				return TextResult->EvaluateAsBasicStringExpression(&InContext);
			}
		}
		else if (OutErrorText)
		{
			*OutErrorText = EvalResult.GetError().Text;
		}
	}

	return false;
}
