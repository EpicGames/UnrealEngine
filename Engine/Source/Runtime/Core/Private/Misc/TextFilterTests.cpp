// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "UObject/NameTypes.h"
#include "Containers/Map.h"
#include "Internationalization/Text.h"
#include "Misc/AutomationTest.h"
#include "Misc/TextFilterExpressionEvaluator.h"
#include "Misc/TextFilter.h"

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

namespace TextFilterTests
{
	struct FTestFilterItem : public ITextFilterExpressionContext
	{
		FTestFilterItem()
		{
			BasicStrings.Add(TEXT("Wooble"));
			BasicStrings.Add(TEXT("Flibble"));
			BasicStrings.Add(TEXT("Type'/Path/To/Asset.Asset'"));
			BasicStrings.Add(TEXT("Other'/Path/To/Asset.Asset'FollowingText"));
			BasicStrings.Add(TEXT("Funky<String>"));
			BasicStrings.Add(TEXT("My-Item"));
			BasicStrings.Add(TEXT("My+Item"));
			BasicStrings.Add(TEXT("My.Item"));

			KeyValuePairs.Add("StringKey", TEXT("Test"));
			KeyValuePairs.Add("IntKey", TEXT("123"));
			KeyValuePairs.Add("FloatKey", TEXT("456.789"));
			KeyValuePairs.Add("NegFloatKey", TEXT("-456.789"));
		}

		// ITextFilterExpressionContext API - used for testing FTextFilterExpressionEvaluator
		virtual bool TestBasicStringExpression(const FTextFilterString& InValue, const ETextFilterTextComparisonMode InTextComparisonMode) const override
		{
			for (const FString& BasicString : BasicStrings)
			{
				if (TextFilterUtils::TestBasicStringExpression(BasicString, InValue, InTextComparisonMode))
				{
					return true;
				}
			}
			return false;
		}
		virtual bool TestComplexExpression(const FName& InKey, const FTextFilterString& InValue, const ETextFilterComparisonOperation InComparisonOperation, const ETextFilterTextComparisonMode InTextComparisonMode) const override
		{
			const FString* ItemValue = KeyValuePairs.Find(InKey);
			if (ItemValue)
			{
				return TextFilterUtils::TestComplexExpression(*ItemValue, InValue, InComparisonOperation, InTextComparisonMode);
			}
			return false;
		}

		// TTextFilter API - used for testing TTextFilter
		static void ExtractItemStrings(const FTestFilterItem* InItem, TArray<FString>& OutStrings)
		{
			OutStrings = InItem->BasicStrings;
		}
		static bool TestItemComplexExpression(const FTestFilterItem* InItem, const FName& InKey, const FTextFilterString& InValue, ETextFilterComparisonOperation InComparisonOperation, ETextFilterTextComparisonMode InTextComparisonMode)
		{
			return InItem->TestComplexExpression(InKey, InValue, InComparisonOperation, InTextComparisonMode);
		}

		TArray<FString> BasicStrings;
		TMap<FName, FString> KeyValuePairs;
	};

	struct ITestFilterExpression
	{
		virtual ~ITestFilterExpression() {}
		virtual bool TestFilterExpression(const TCHAR* InFilterExpression, const bool bExpected) = 0;
	};

	struct FTestFilterExpression : public ITestFilterExpression
	{
		FTestFilterExpression(FAutomationTestBase* InTest, const FTestFilterItem* InTestItem)
			: Test(InTest)
			, TestItem(InTestItem)
		{
		}

		virtual bool TestFilterExpression(const TCHAR* InFilterExpression, const bool bExpected) override
		{
			FText FilterErrorText;
			const bool bActual = TestFilterExpressionImpl(InFilterExpression, TestItem, FilterErrorText);

			bool bResult = true;

			if (!FilterErrorText.IsEmpty())
			{
				Test->AddError(FString::Printf(TEXT("Filter expression '%s' reported an error: %s"), InFilterExpression, *FilterErrorText.ToString()));
				bResult = false;
			}

			if (bActual != bExpected)
			{
				Test->AddError(FString::Printf(TEXT("Filter expression '%s' evaluated incorrectly: Expected: %s, Actual: %s"), 
					InFilterExpression, 
					(bExpected) ? TEXT("true") : TEXT("false"), 
					(bActual) ? TEXT("true") : TEXT("false"))
					);
				bResult = false;
			}

			return bResult;
		}

		virtual bool TestFilterExpressionImpl(const TCHAR* InFilterExpression, const FTestFilterItem* InTestItem, FText& OutErrorText) = 0;

		FAutomationTestBase* Test;
		const FTestFilterItem* TestItem;
	};

	struct FTestFilterExpression_TextFilterExpressionEvaluator : public FTestFilterExpression
	{
		FTestFilterExpression_TextFilterExpressionEvaluator(FAutomationTestBase* InTest, const FTestFilterItem* InTestItem, FTextFilterExpressionEvaluator* InTextFilterExpressionEvaluator)
			: FTestFilterExpression(InTest, InTestItem)
			, TextFilterExpressionEvaluator(InTextFilterExpressionEvaluator)
		{
		}

		virtual bool TestFilterExpressionImpl(const TCHAR* InFilterExpression, const FTestFilterItem* InTestItem, FText& OutErrorText) override
		{
			TextFilterExpressionEvaluator->SetFilterText(FText::FromString(InFilterExpression));
			OutErrorText = TextFilterExpressionEvaluator->GetFilterErrorText();
			return TextFilterExpressionEvaluator->TestTextFilter(*InTestItem);
		}

		FTextFilterExpressionEvaluator* TextFilterExpressionEvaluator;
	};

	struct FTestFilterExpression_TextFilter : public FTestFilterExpression
	{
		FTestFilterExpression_TextFilter(FAutomationTestBase* InTest, const FTestFilterItem* InTestItem, TTextFilter<const FTestFilterItem*>* InTextFilter)
			: FTestFilterExpression(InTest, InTestItem)
			, TextFilter(InTextFilter)
		{
		}

		virtual bool TestFilterExpressionImpl(const TCHAR* InFilterExpression, const FTestFilterItem* InTestItem, FText& OutErrorText) override
		{
			TextFilter->SetRawFilterText(FText::FromString(InFilterExpression));
			OutErrorText = TextFilter->GetFilterErrorText();
			return TextFilter->PassesFilter(InTestItem);
		}

		TTextFilter<const FTestFilterItem*>* TextFilter;
	};

	bool TestAllCommonFilterExpressions(ITestFilterExpression& InTestPayload)
	{
		bool bResult = true;

		bResult &= InTestPayload.TestFilterExpression(TEXT("Wooble"), true);
		bResult &= InTestPayload.TestFilterExpression(TEXT("Woo..."), true);
		bResult &= InTestPayload.TestFilterExpression(TEXT("...ble"), true);
		bResult &= InTestPayload.TestFilterExpression(TEXT("Wo... AND ...le"), true);
		bResult &= InTestPayload.TestFilterExpression(TEXT("'Wooble'"), true);
		bResult &= InTestPayload.TestFilterExpression(TEXT("'Woo'"), true);
		bResult &= InTestPayload.TestFilterExpression(TEXT("+'Wooble'"), true);
		bResult &= InTestPayload.TestFilterExpression(TEXT("+'Woo'"), false);
		bResult &= InTestPayload.TestFilterExpression(TEXT("+Wooble"), true);
		bResult &= InTestPayload.TestFilterExpression(TEXT("+Woo"), false);
		bResult &= InTestPayload.TestFilterExpression(TEXT("Wooble2"), false);
		bResult &= InTestPayload.TestFilterExpression(TEXT("-Wooble2"), true);
		bResult &= InTestPayload.TestFilterExpression(TEXT("Wooble2 Flibble"), false);
		bResult &= InTestPayload.TestFilterExpression(TEXT("Wooble2 OR Flibble"), true);
		bResult &= InTestPayload.TestFilterExpression(TEXT("Wooble2 AND Flibble"), false);
		bResult &= InTestPayload.TestFilterExpression(TEXT("Wooble && !Flibble"), false);
		bResult &= InTestPayload.TestFilterExpression(TEXT("Flibble -Wooble2"), true);
		bResult &= InTestPayload.TestFilterExpression(TEXT("Flibble OR -Wooble2"), true);
		bResult &= InTestPayload.TestFilterExpression(TEXT("Flibble AND -Wooble2"), true);
		bResult &= InTestPayload.TestFilterExpression(TEXT("Type'/Path/To/Asset.Asset'"), true);
		bResult &= InTestPayload.TestFilterExpression(TEXT("Other'/Path/To/Asset.Asset'FollowingText"), true);
		bResult &= InTestPayload.TestFilterExpression(TEXT("\"Funky<String>\""), true);
		bResult &= InTestPayload.TestFilterExpression(TEXT("My-Item"), true);
		bResult &= InTestPayload.TestFilterExpression(TEXT("+My-Item"), true);
		bResult &= InTestPayload.TestFilterExpression(TEXT("My+Item"), true);
		bResult &= InTestPayload.TestFilterExpression(TEXT("+My+Item"), true);
		bResult &= InTestPayload.TestFilterExpression(TEXT("My.Item"), true);
		bResult &= InTestPayload.TestFilterExpression(TEXT("+My.Item"), true);

		return bResult;
	}

	bool TestAllBasicFilterExpressions(ITestFilterExpression& InTestPayload)
	{
		bool bResult = true;

		bResult &= TestAllCommonFilterExpressions(InTestPayload);
		bResult &= InTestPayload.TestFilterExpression(TEXT("Funky<String>"), true);

		return bResult;
	}

	bool TestAllComplexFilterExpressions(ITestFilterExpression& InTestPayload)
	{
		bool bResult = true;

		bResult &= TestAllCommonFilterExpressions(InTestPayload);
		bResult &= InTestPayload.TestFilterExpression(TEXT("'Funky<String>'"), true);
		bResult &= InTestPayload.TestFilterExpression(TEXT("StringKey=Test"), true);
		bResult &= InTestPayload.TestFilterExpression(TEXT("StringKey!=Test"), false);
		bResult &= InTestPayload.TestFilterExpression(TEXT("IntKey=123"), true);
		bResult &= InTestPayload.TestFilterExpression(TEXT("IntKey>122"), true);
		bResult &= InTestPayload.TestFilterExpression(TEXT("IntKey<122"), false);
		bResult &= InTestPayload.TestFilterExpression(TEXT("FloatKey=456.789"), true);
		bResult &= InTestPayload.TestFilterExpression(TEXT("FloatKey>456"), true);
		bResult &= InTestPayload.TestFilterExpression(TEXT("FloatKey<456"), false);
		bResult &= InTestPayload.TestFilterExpression(TEXT("NegFloatKey=-456.789"), true);
		bResult &= InTestPayload.TestFilterExpression(TEXT("NegFloatKey>-456"), false);
		bResult &= InTestPayload.TestFilterExpression(TEXT("NegFloatKey<-456"), true);
		bResult &= InTestPayload.TestFilterExpression(TEXT("IntKey==300 || FloatKey==456.789"), true);
		bResult &= InTestPayload.TestFilterExpression(TEXT("IntKey==300 && FloatKey==456.789"), false);
		bResult &= InTestPayload.TestFilterExpression(TEXT("(IntKey==300 && FloatKey==456.789) OR StringKey==Test"), true);

		return bResult;
	}
};

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FTextFilterTests_TextFilterExpressionEvaluator, "System.Core.Text.TextFilterExpressionEvaluator", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
bool FTextFilterTests_TextFilterExpressionEvaluator::RunTest(const FString& Parameters)
{
	using namespace TextFilterTests;

	FTestFilterItem TestItem;

	bool bResult = true;

	// Test basic filtering
	{
		FTextFilterExpressionEvaluator TextFilterExpressionEvaluator(ETextFilterExpressionEvaluatorMode::BasicString);
		FTestFilterExpression_TextFilterExpressionEvaluator TestFilterExpressionContext(this, &TestItem, &TextFilterExpressionEvaluator);
		bResult &= TestAllBasicFilterExpressions(TestFilterExpressionContext);
	}

	// Test complex filtering
	{
		FTextFilterExpressionEvaluator TextFilterExpressionEvaluator(ETextFilterExpressionEvaluatorMode::Complex);
		FTestFilterExpression_TextFilterExpressionEvaluator TestFilterExpressionContext(this, &TestItem, &TextFilterExpressionEvaluator);
		bResult &= TestAllComplexFilterExpressions(TestFilterExpressionContext);
	}

	return bResult;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FTextFilterTests_TextFilter, "System.Core.Text.TextFilter", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
bool FTextFilterTests_TextFilter::RunTest(const FString& Parameters)
{
	using namespace TextFilterTests;

	FTestFilterItem TestItem;

	bool bResult = true;

	// Test basic filtering
	{
		TTextFilter<const FTestFilterItem*> TextFilter(
			TTextFilter<const FTestFilterItem*>::FItemToStringArray::CreateStatic(&FTestFilterItem::ExtractItemStrings)
			);
		FTestFilterExpression_TextFilter TestFilterExpressionContext(this, &TestItem, &TextFilter);
		bResult &= TestAllBasicFilterExpressions(TestFilterExpressionContext);
	}

	// Test complex filtering
	{
		TTextFilter<const FTestFilterItem*> TextFilter(
			TTextFilter<const FTestFilterItem*>::FItemToStringArray::CreateStatic(&FTestFilterItem::ExtractItemStrings), 
			TTextFilter<const FTestFilterItem*>::FItemTestComplexExpression::CreateStatic(&FTestFilterItem::TestItemComplexExpression)
			);
		FTestFilterExpression_TextFilter TestFilterExpressionContext(this, &TestItem, &TextFilter);
		bResult &= TestAllComplexFilterExpressions(TestFilterExpressionContext);
	}

	return bResult;
}

#endif //!(UE_BUILD_SHIPPING || UE_BUILD_TEST)
