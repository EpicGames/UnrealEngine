// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/TextFilterUtils.h"

#include "HAL/PlatformString.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/Char.h"
#include "Templates/UnrealTemplate.h"

namespace TextFilterInternal
{
	template <typename CharType>
	bool CompareStrings(const CharType* Str, int32 Length, const CharType* Find, int32 FindLength, const ETextFilterTextComparisonMode InTextComparisonMode, ESearchCase::Type SearchCase = ESearchCase::CaseSensitive)
	{
		if (SearchCase == ESearchCase::CaseSensitive)
		{
			switch(InTextComparisonMode)
			{
			case ETextFilterTextComparisonMode::Exact:
				return TCString<CharType>::Strcmp(Str, Find) == 0;
			case ETextFilterTextComparisonMode::Partial:
				return TCString<CharType>::Strstr(Str, Find) != nullptr;
			case ETextFilterTextComparisonMode::StartsWith:
				if (FindLength < 0)
				{
					FindLength = TCString<CharType>::Strlen(Find);
				}
				return FindLength > 0 && TCString<CharType>::Strncmp(Str, Find, FindLength) == 0;
			case ETextFilterTextComparisonMode::EndsWith:
				if (Length < 0)
				{
					Length = TCString<CharType>::Strlen(Str);
				}
				if (FindLength < 0)
				{
					FindLength = TCString<CharType>::Strlen(Find);
				}
				return Length > 0 && Length >= FindLength
					&& TCString<CharType>::Strncmp(Str + (Length - FindLength), Find, FindLength) == 0;
			default:
				break;
			}
		}
		else
		{
			switch(InTextComparisonMode)
			{
			case ETextFilterTextComparisonMode::Exact:
				return TCString<CharType>::Stricmp(Str, Find) == 0;
			case ETextFilterTextComparisonMode::Partial:
				return TCString<CharType>::Stristr(Str, Find) != nullptr;
			case ETextFilterTextComparisonMode::StartsWith:
				if (FindLength < 0)
				{
					FindLength = TCString<CharType>::Strlen(Find);
				}
				return FindLength > 0 && TCString<CharType>::Strnicmp(Str, Find, FindLength) == 0;
			case ETextFilterTextComparisonMode::EndsWith:
				if (Length < 0)
				{
					Length = TCString<CharType>::Strlen(Str);
				}
				if (FindLength < 0)
				{
					FindLength = TCString<CharType>::Strlen(Find);
				}
				return Length > 0 && Length >= FindLength
					&& TCString<CharType>::Strnicmp(Str + (Length - FindLength), Find, FindLength) == 0;
			default:
				break;
			}
		}

		return false;
	}
} // namespace TextFilterInternal

/** ToUpper implementation that's optimized for ASCII characters */
namespace FastToUpper
{
	// -32 = 'A' - 'a'
	static const int32 ToUpperAdjustmentTable[] = 
	{
		// ASCII control characters
		0,		// NUL
		0,		// SOH
		0,		// STX
		0,		// ETX
		0,		// EOT
		0,		// ENQ
		0,		// ACK
		0,		// BEL
		0,		// BS
		0,		// HT
		0,		// LF
		0,		// VT
		0,		// FF
		0,		// CR
		0,		// SO
		0,		// SI
		0,		// DLE
		0,		// DC1
		0,		// DC2
		0,		// DC3
		0,		// DC4
		0,		// NAK
		0,		// SYN
		0,		// ETB
		0,		// CAN
		0,		// EM
		0,		// SUB
		0,		// ESC
		0,		// FS
		0,		// GS
		0,		// RS
		0,		// US

		// ASCII printable characters
		0,		//  
		0,		// !
		0,		// "
		0,		// #
		0,		// $
		0,		// %
		0,		// &
		0,		// '
		0,		// (
		0,		// )
		0,		// *
		0,		// +
		0,		// ,
		0,		// -
		0,		// .
		0,		// /
		0,		// 0
		0,		// 1
		0,		// 2
		0,		// 3
		0,		// 4
		0,		// 5
		0,		// 6
		0,		// 7
		0,		// 8
		0,		// 9
		0,		// :
		0,		// ;
		0,		// <
		0,		// =
		0,		// >
		0,		// ?
		0,		// @
		0,		// A
		0,		// B
		0,		// C
		0,		// D
		0,		// E
		0,		// F
		0,		// G
		0,		// H
		0,		// I
		0,		// J
		0,		// K
		0,		// L
		0,		// M
		0,		// N
		0,		// O
		0,		// P
		0,		// Q
		0,		// R
		0,		// S
		0,		// T
		0,		// U
		0,		// V
		0,		// W
		0,		// X
		0,		// Y
		0,		// Z
		0,		// [
		0,		// Backslash
		0,		// ]
		0,		// ^
		0,		// _
		0,		// `
		-32,	// a
		-32,	// b
		-32,	// c
		-32,	// d
		-32,	// e
		-32,	// f
		-32,	// g
		-32,	// h
		-32,	// i
		-32,	// j
		-32,	// k
		-32,	// l
		-32,	// m
		-32,	// n
		-32,	// o
		-32,	// p
		-32,	// q
		-32,	// r
		-32,	// s
		-32,	// t
		-32,	// u
		-32,	// v
		-32,	// w
		-32,	// x
		-32,	// y
		-32,	// z
		0,		// {
		0,		// |
		0,		// }
		0,		// ~
		0,		// 	
	};
	static const int32 ToUpperAdjustmentTableCount = UE_ARRAY_COUNT(ToUpperAdjustmentTable);

	FORCEINLINE TCHAR ToUpper(const TCHAR InChar)
	{
		if (InChar < ToUpperAdjustmentTableCount)
		{
			return (TCHAR)(InChar + ToUpperAdjustmentTable[(int32)InChar]);
		}
		return FChar::ToUpper(InChar);
	}
} // namespace FastToUpper

FTextFilterString::FTextFilterString() = default;

FTextFilterString::FTextFilterString(const FTextFilterString& Other) = default;

FTextFilterString::FTextFilterString(FTextFilterString&& Other) = default;

FTextFilterString::FTextFilterString(const FString& InString)
	: InternalString(InString)
{
	UppercaseInternalString();
}

FTextFilterString::FTextFilterString(FString&& InString)
	: InternalString(MoveTemp(InString))
{
	UppercaseInternalString();
}

FTextFilterString::FTextFilterString(FStringView InString)
	: InternalString(InString)
{
	UppercaseInternalString();
}

FTextFilterString::FTextFilterString(const TCHAR* InString)
	: InternalString(InString)
{
	UppercaseInternalString();
}

FTextFilterString::FTextFilterString(FName InName)
{
	InName.AppendString(InternalString);
	UppercaseInternalString();
}

FTextFilterString& FTextFilterString::operator=(const FTextFilterString& Other) = default;

FTextFilterString& FTextFilterString::operator=(FTextFilterString&& Other) = default;

FTextFilterString& FTextFilterString::operator=(const FString& InString)
{
	InternalString = InString;
	UppercaseInternalString();
	return *this;
}

FTextFilterString& FTextFilterString::operator=(FStringView InString)
{
	InternalString = InString;
	UppercaseInternalString();
	return *this;
}

FTextFilterString& FTextFilterString::operator=(const TCHAR* InString)
{
	InternalString = InString;
	UppercaseInternalString();
	return *this;
}

FTextFilterString& FTextFilterString::operator=(FName InName)
{
	InternalString.Reset();
	InName.AppendString(InternalString);
	UppercaseInternalString();
	return *this;
}

bool FTextFilterString::CompareText(const FTextFilterString& InOther, const ETextFilterTextComparisonMode InTextComparisonMode) const
{
	return TextFilterInternal::CompareStrings<TCHAR>(*InternalString, InternalString.Len(), *InOther.InternalString, InOther.InternalString.Len(), InTextComparisonMode);
}

bool FTextFilterString::CompareFString(const FString& InOtherUpper, const ETextFilterTextComparisonMode InTextComparisonMode) const
{
	return TextFilterInternal::CompareStrings<TCHAR>(*InOtherUpper, InOtherUpper.Len(), *InternalString, InternalString.Len(), InTextComparisonMode);
}

bool FTextFilterString::CompareFStringView(FStringView InOtherUpper, const ETextFilterTextComparisonMode InTextComparisonMode) const
{
	return TextFilterInternal::CompareStrings<TCHAR>(InOtherUpper.GetData(), InOtherUpper.Len(), *InternalString, InternalString.Len(), InTextComparisonMode);
}

bool FTextFilterString::CompareName(const FName& InOther, const ETextFilterTextComparisonMode InTextComparisonMode) const
{
	TextFilterUtils::FNameBufferWithNumber OtherNameBuffer(InOther);
	if (OtherNameBuffer.IsWide())
	{
		FCString::Strupr(OtherNameBuffer.GetWideNamePtr(), OtherNameBuffer.GetMaxBufferLength());
		return TextFilterInternal::CompareStrings<TCHAR>(OtherNameBuffer.GetWideNamePtr(), -1, *InternalString, InternalString.Len(), InTextComparisonMode);
	}
	else if (InternalStringAnsi.Num() > 1)
	{
		FCStringAnsi::Strupr(OtherNameBuffer.GetAnsiNamePtr(), OtherNameBuffer.GetMaxBufferLength());
		return TextFilterInternal::CompareStrings<ANSICHAR>(OtherNameBuffer.GetAnsiNamePtr(), -1, InternalStringAnsi.GetData(), InternalStringAnsi.Num() - 1, InTextComparisonMode);
	}

	return false;
}

bool FTextFilterString::CanCompareNumeric(const FTextFilterString& InOther) const
{
	return InternalString.IsNumeric() && InOther.InternalString.IsNumeric();
}

bool FTextFilterString::CompareNumeric(const FTextFilterString& InOther, const ETextFilterComparisonOperation InComparisonOperation) const
{
	const double OurNumericValue = FCString::Atod(*InternalString);
	const double OtherNumericValue = FCString::Atod(*InOther.InternalString);
	const double Difference = OurNumericValue - OtherNumericValue;

	const int32 ComparisonSign = (int32)FMath::Sign(Difference);
	switch (InComparisonOperation)
	{
	case ETextFilterComparisonOperation::Equal:
		return ComparisonSign == 0;
	case ETextFilterComparisonOperation::NotEqual:
		return ComparisonSign != 0;
	case ETextFilterComparisonOperation::Less:
		return ComparisonSign < 0;
	case ETextFilterComparisonOperation::LessOrEqual:
		return ComparisonSign <= 0;
	case ETextFilterComparisonOperation::Greater:
		return ComparisonSign > 0;
	case ETextFilterComparisonOperation::GreaterOrEqual:
		return ComparisonSign >= 0;
	default:
		check(false);
	};

	return false;
}

void FTextFilterString::UppercaseInternalString()
{
	for (TCHAR* CharPtr = const_cast<TCHAR*>(*InternalString); *CharPtr; ++CharPtr)
	{
		*CharPtr = FastToUpper::ToUpper(*CharPtr);
	}

	TextFilterUtils::TryConvertWideToAnsi(InternalString, InternalStringAnsi);
}

bool TextFilterUtils::TryConvertWideToAnsi(const FString& SourceWideString, TArray<ANSICHAR>& DestAnsiString)
{
	DestAnsiString.Reset();

	if (FCString::IsPureAnsi(*SourceWideString))
	{
		if (SourceWideString.Len() > 0)
		{
			DestAnsiString.AddUninitialized(SourceWideString.Len() + 1);
			FPlatformString::Convert(DestAnsiString.GetData(), DestAnsiString.Num(), *SourceWideString, SourceWideString.Len() + 1);
		}

		return true;
	}

	return false;
}

int32 TextFilterUtils::NameStrincmp(const FName& Name, const FString& WideOther, const TArray<ANSICHAR>& AnsiOther, int32 Length)
{
	TextFilterUtils::FNameBufferWithNumber NameBuffer(Name);
	if (NameBuffer.IsWide())
	{
		return FCStringWide::Strnicmp(NameBuffer.GetWideNamePtr(), *WideOther, Length);
	}
	else if (AnsiOther.Num() > 1)
	{
		return FCStringAnsi::Strnicmp(NameBuffer.GetAnsiNamePtr(), AnsiOther.GetData(), Length);
	}
	else
	{
		// We know they are not equal (FName contains only ansi while other contains wide)
		return -1;
	}
}

bool TextFilterUtils::TestBasicStringExpression(const FTextFilterString& InValue1, const FTextFilterString& InValue2, const ETextFilterTextComparisonMode InTextComparisonMode)
{
	return InValue1.CompareText(InValue2, InTextComparisonMode);
}

bool TextFilterUtils::TestComplexExpression(const FTextFilterString& InValue1, const FTextFilterString& InValue2, const ETextFilterComparisonOperation InComparisonOperation, const ETextFilterTextComparisonMode InTextComparisonMode)
{
	if (InValue1.CanCompareNumeric(InValue2))
	{
		return InValue1.CompareNumeric(InValue2, InComparisonOperation);
	}

	// Text can only work with Equal or NotEqual type tests
	if (InComparisonOperation == ETextFilterComparisonOperation::Equal || InComparisonOperation == ETextFilterComparisonOperation::NotEqual)
	{
		const bool bIsMatch = InValue1.CompareText(InValue2, InTextComparisonMode);
		return (InComparisonOperation == ETextFilterComparisonOperation::Equal) ? bIsMatch : !bIsMatch;
	}

	return false;
}
