// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/TimeGuard.h"
#include "Misc/ScopeLock.h"

DEFINE_LOG_CATEGORY_STATIC(LogTimeGuard, Log, All);

#if DO_TIMEGUARD

TMap<const TCHAR*, FTimeGuard::FGuardInfo, FDefaultSetAllocator, TStringPointerMapKeyFuncs_DEPRECATED<const TCHAR*, FGuardInfo>>  FTimeGuard::HitchData;
bool FTimeGuard::bEnabled;
float FTimeGuard::FrameTimeThresholdMS = 1000.0 / 30.0;
FCriticalSection FTimeGuard::ReportMutex;
TSet<const TCHAR *, TStringPointerSetKeyFuncs_DEPRECATED<const TCHAR*>> FTimeGuard::VolatileNames; // any names which come in volatile we allocate them to a static string and put them in this array


void FTimeGuard::SetEnabled(bool InEnable)
{
	bEnabled = InEnable;
}

void FTimeGuard::SetFrameTimeThresholdMS(float InTimeMS)
{
	FrameTimeThresholdMS = InTimeMS;
}

void FTimeGuard::ClearData()
{
	FScopeLock Lock(&ReportMutex);
	HitchData.Empty();
}

void FTimeGuard::GetData(TMap<const TCHAR*, FGuardInfo, FDefaultSetAllocator, TStringPointerMapKeyFuncs_DEPRECATED<const TCHAR*, FGuardInfo>>& Dest)
{
	FScopeLock Lock(&ReportMutex);
	Dest = HitchData;
}

void FTimeGuard::ReportHitch(const TCHAR* VolatileInName, const float TimeMS, bool VolatileName)
{
	FScopeLock lock(&ReportMutex);

	const TCHAR* InName = VolatileInName;
	if ( VolatileName )
	{
		const TCHAR** CachedName = VolatileNames.Find(VolatileInName);
		if ( CachedName == nullptr )
		{
			int32 StringLength = FCString::Strlen(VolatileInName) + 1;
			TCHAR *NewString = new TCHAR[StringLength];
			FCString::Strcpy(NewString, StringLength, VolatileInName);
			VolatileNames.Add(NewString);
			InName = NewString;
		}
		else
		{
			InName = *CachedName;
		}
	}

	FGuardInfo& Data = HitchData.FindOrAdd(InName);

	if (Data.Count == 0)
	{
		Data.FirstTime = FDateTime::UtcNow();
	}

	Data.Count++;
	Data.Total += TimeMS;
	Data.Min = FMath::Min(Data.Min, TimeMS);
	Data.Max = FMath::Max(Data.Max, TimeMS);
	Data.LastTime = FDateTime::UtcNow();

	UE_LOGF(LogTimeGuard, Warning, "Detected Hitch of %0.2fms in %ls", TimeMS, InName);
}

#endif // DO_TIMEGUARD
