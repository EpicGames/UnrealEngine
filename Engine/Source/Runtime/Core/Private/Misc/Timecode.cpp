// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/Timecode.h"
#include "HAL/IConsoleManager.h"


static TAutoConsoleVariable<bool> CVarUseDropFormatTimecodeByDefaultWhenSupported(
	TEXT("timecode.UseDropFormatTimecodeByDefaultWhenSupported"),
	1.0f,
	TEXT("By default, should we generate a timecode in drop frame format when the frame rate does support it."),
	ECVF_Default);


static TAutoConsoleVariable<int32> CVarForceStringifyTimecodeSubframes(
	TEXT("timecode.ForceStringifyTimecodeSubframes"),
	0,
	TEXT("Should Timecode.ToString() be forced to include subframes. 0 - Don't force. 1 - Force show. 2 - Force hide"),
	ECVF_Default);

/* FTimecode interface
 *****************************************************************************/
bool FTimecode::UseDropFormatTimecodeByDefaultWhenSupported()
{
	return CVarUseDropFormatTimecodeByDefaultWhenSupported.GetValueOnAnyThread();
}

namespace UE::Core::Private
{
	/**
	 * Parses a string representation of a timecode.
	 *
	 * Supported SMPTE drop frame (DF) and non-drop frame (NDF) formats variations:
	 * - NDF: HH:MM:SS:FF
	 * - DF: HH:MM:SS;FF or HH:MM:SS.FF or HH;MM;SS;FF or HH.MM.SS.FF.
	 *
	 * The 2 digits per number is not enforced. It is possible to parse high frame numbers (above 60), such as for audio timecodes.
	 * Full SMPTE compliance is not ensured by the parser (i.e. greater than 24h, negative time and any number of frames per second).
	 *
	 * Sub-frame variation:
	 * Supports the sub-frame variation where the frame number is a decimal number: HH:MM:SS:FF.ZZ.
	 *
	 * Side effect of supporting sub-frame is that this parser can't unambiguously parse partial timecodes.
	 * For instance, "00:00.00" is ambiguous because it is undetermined if the last value is frame or sub-frame.
	 */
	class FTimecodeParser
	{
	public:		
		// Expected frame separators
		const TArray<TCHAR, TInlineAllocator<3>> AllSeparators;
		const TArray<TCHAR, TInlineAllocator<3>> SubFrameSeparators;
		const TArray<TCHAR, TInlineAllocator<3>> DropFrameSeparators;

		// Set of separators for each positions.
		TArray<const TArray<TCHAR, TInlineAllocator<3>>*, TInlineAllocator<4>> ValueSeparators;

		FTimecodeParser()
			: AllSeparators({ TEXT(':'), TEXT(';'), TEXT('.') })
			, SubFrameSeparators({TEXT('.')})
			, DropFrameSeparators({ TEXT(';'), TEXT('.') })
		{
			// Expected separators for each positions: HH[:;.]MM[:;,]SS[:;.]FF.DD
			ValueSeparators.Add(&AllSeparators);
			ValueSeparators.Add(&AllSeparators);
			ValueSeparators.Add(&AllSeparators);
			ValueSeparators.Add(&SubFrameSeparators);
		}

		TOptional<FTimecode> Evaluate(const FStringView InString) const
		{
			TArray<TCHAR, TInlineAllocator<64>> Buffer;
			constexpr int32 MaxNumberOfValues = 5;
			constexpr int32 MinNumberOfValues = 4;
			TArray<double, TInlineAllocator<MaxNumberOfValues>> Values;
			TArray<int32, TInlineAllocator<MaxNumberOfValues>> Signs;

			// Separator positions: HH(0):MM(1):SS(2):FF(3).DD
			constexpr int32 FrameSeparatorPosition = 2;
			constexpr int32 SubFrameSeparatorPosition = 3;

			bool bDropFrameFormat = false;
			bool bDropFrameSeparatorEncountered = false;

			auto TryPushValue = [&Buffer, &Values, &Signs]() -> bool
			{
				Buffer.Push(TEXT('\0'));

				// Validation
				if (!FCString::IsNumeric(Buffer.GetData()))
				{
					return false;
				}

				// Keep track of sign independently of values because Atoi can't represent -0;
				Signs.Push(Buffer[0] == TEXT('-') ? -1 : 1);

				if (Values.Num() >= MinNumberOfValues)
				{
					// Sub-frame fraction is parsed as a positive-only decimal fraction.
					// It will be made negative only if there is no other way to preserve the sign (i.e all other values are zero).
					Buffer.RemoveAll([](TCHAR InChar) { return InChar == TEXT('-') || InChar == TEXT('+'); });
					Buffer.Insert(TEXT('.'), 0);
					Values.Push(FCString::Atof(Buffer.GetData()));
				}
				else
				{
					// hours, minutes, seconds and frames are integers
					Values.Push(FCString::Atoi(Buffer.GetData()));
				}

				Buffer.Reset();
				return true;
			};

			// Parse the string and convert values.
			for (const TCHAR Char : InString)
			{
				if (AllSeparators.Contains(Char))
				{
					const uint32 SeparatorPosition = Values.Num();
					
					// Validate expected separator for this value position.
					if (!ValueSeparators.IsValidIndex(SeparatorPosition) || !ValueSeparators[SeparatorPosition]->Contains(Char))
					{
						// Unexpected separator.
						return {};
					}

					if (Buffer.IsEmpty() || !TryPushValue())
					{
						// value was not a number (or empty)
						return {};
					}

					// The DF separator can be between all digit pairs or just the 3rd position.
					// However, if it is present but not at the 3rd position, this is ambiguous.
					if (DropFrameSeparators.Contains(Char) && SeparatorPosition < SubFrameSeparatorPosition)
					{
						bDropFrameSeparatorEncountered = true;	// This will detect ambiguous cases.
					
						// Encountering ';' or '.' on the 3rd separator is a confirmed DF format.
						if (SeparatorPosition == FrameSeparatorPosition)
						{
							bDropFrameFormat = true;
						}
					}
				}
				else if (!FChar::IsWhitespace(Char)) // Ignore whitespaces
				{
					Buffer.Push(Char);
				}
			}

			// Reject ambiguous separator combinations (DF vs NDF).
			if (bDropFrameSeparatorEncountered && !bDropFrameFormat)
			{
				return {};
			}
			
			// Convert last value. Fail on empty or non-numeric string.
			if (Buffer.IsEmpty() || !TryPushValue())
			{
				return {};
			}

			// Validate that we have the expected number of parsed values.
			if (Values.Num() < MinNumberOfValues || Values.Num() > MaxNumberOfValues)
			{
				return {};
			}

			// Ensure the sign is preserved in case of zero values.
			if (Signs.Contains(-1))
			{
				// A negative sign has been encountered while parsing, see if we have a negative value already.
				const bool bFoundNegative = Values.ContainsByPredicate([](double InValue) { return InValue < 0.0; });

				// Here, we are doing our best to preserve the sign by applying it to the first non-zero value.
				if (!bFoundNegative)
				{
					if (double* FirstNonZeroValue = Values.FindByPredicate([](double InValue) { return InValue > 0.0; }))
					{
						*FirstNonZeroValue = -*FirstNonZeroValue;
					}
				}
			}

			FTimecode TC;
			TC.Hours = static_cast<int32>(Values[0]);
			TC.Minutes = static_cast<int32>(Values[1]);
			TC.Seconds = static_cast<int32>(Values[2]);
			TC.Frames = static_cast<int32>(Values[3]);

			if (Values.IsValidIndex(4))
			{
				// Note: For valid timecode math, it is necessary to allow a negative sub-frame fraction (if all other values are zero).
				TC.Subframe = FMath::Clamp(static_cast<float>(Values[4]), -1.0f, 1.0f);
			}

			TC.bDropFrameFormat = bDropFrameFormat;

			return TC;
		}
		
		static FTimecodeParser& Get()
		{
			static FTimecodeParser StaticTimecodeParser;
			return StaticTimecodeParser;
		}
	};
}

TOptional<FTimecode> FTimecode::ParseTimecode(const FStringView InTimecodeString)
{
	using namespace UE::Core::Private;
	FTimecodeParser& StaticTimecodeParser = FTimecodeParser::Get();
	return StaticTimecodeParser.Evaluate(InTimecodeString);
}


FString FTimecode::ToString(bool bForceSignDisplay, bool bDisplaySubframe) const
{
	bool bHasNegativeComponent = Hours < 0 || Minutes < 0 || Seconds < 0 || Frames < 0;

	const TCHAR* NegativeSign = TEXT("- ");
	const TCHAR* PositiveSign = TEXT("+ ");
	const TCHAR* SignText = TEXT("");

	if (bHasNegativeComponent)
	{
		SignText = NegativeSign;
	}
	else if (bForceSignDisplay)
	{
		SignText = PositiveSign;
	}

	// Use a buffer that will hold 64 chars to account for maximum int sizes for hours, min, seconds, frames + 3 charcs for subframe +/- 3 chars for sign.
	//
	TStringBuilder<64> Builder;
	if (bDropFrameFormat)
	{
		Builder.Appendf(TEXT("%s%02d:%02d:%02d;%02d"), SignText, FMath::Abs(Hours), FMath::Abs(Minutes), FMath::Abs(Seconds), FMath::Abs(Frames));
	}
	else
	{
		Builder.Appendf(TEXT("%s%02d:%02d:%02d:%02d"), SignText, FMath::Abs(Hours), FMath::Abs(Minutes), FMath::Abs(Seconds), FMath::Abs(Frames));
	}

	switch (CVarForceStringifyTimecodeSubframes.GetValueOnAnyThread())
	{
	case 1:
		bDisplaySubframe = true;
		break;

	case 2:
		bDisplaySubframe = false;
		break;

	default:
		break;
	}

	if (bDisplaySubframe)
	{
		int32 ClampedSubframe = static_cast<int32>(FMath::Clamp(100 * Subframe, 0, 99));
		Builder.Appendf(TEXT(".%02d"), ClampedSubframe);
	}

	return FString(Builder);
}