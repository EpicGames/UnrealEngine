// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/Timespan.h"

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "HAL/PlatformString.h"
#include "Misc/CString.h"
#include "Serialization/Archive.h"
#include "Templates/TypeHash.h"
#include "UObject/PropertyPortFlags.h"


/* FTimespan interface
 *****************************************************************************/

bool FTimespan::ExportTextItem(FString& ValueStr, FTimespan const& DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const
{
	ValueStr += ToString(TEXT("%D.%h:%m:%s.%n"));

	return true;
}


bool FTimespan::ImportTextItem(const TCHAR*& Buffer, int32 PortFlags, UObject* Parent, FOutputDevice* ErrorText)
{
	const int32 ExportTimespanLen = 27;

	if (FPlatformString::Strlen(Buffer) < ExportTimespanLen)
	{
		return false;
	}

	if (!Parse(FString(Buffer).Left(ExportTimespanLen), *this))
	{
		return false;
	}

	Buffer += ExportTimespanLen;

	return true;
}


bool FTimespan::Serialize(FArchive& Ar)
{
	Ar << *this;

	return true;
}


bool FTimespan::NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess)
{
	Ar << *this;
	bOutSuccess = true;
	return true;
}


FString FTimespan::ToString() const
{
	if (GetDays() == 0)
	{
		return ToString(TEXT("%h:%m:%s.%f"));
	}

	return ToString(TEXT("%d.%h:%m:%s.%f"));
}


FString FTimespan::ToString(const TCHAR* Format) const
{
	FString Result;

	Result += (Ticks < 0) ? TCHAR('-') : TCHAR('+');

	while (*Format != TCHAR('\0'))
	{
		if ((*Format == TCHAR('%')) && (*++Format != TCHAR('\0')))
		{
			switch (*Format)
			{
			case TCHAR('d'): Result += FString::Printf(TEXT("%i"), FMath::Abs(GetDays())); break;
			case TCHAR('D'): Result += FString::Printf(TEXT("%08i"), FMath::Abs(GetDays())); break;
			case TCHAR('h'): Result += FString::Printf(TEXT("%02i"), FMath::Abs(GetHours())); break;
			case TCHAR('m'): Result += FString::Printf(TEXT("%02i"), FMath::Abs(GetMinutes())); break;
			case TCHAR('s'): Result += FString::Printf(TEXT("%02i"), FMath::Abs(GetSeconds())); break;
			case TCHAR('f'): Result += FString::Printf(TEXT("%03i"), FMath::Abs(GetFractionMilli())); break;
			case TCHAR('u'): Result += FString::Printf(TEXT("%06i"), FMath::Abs(GetFractionMicro())); break;
			case TCHAR('t'): Result += FString::Printf(TEXT("%07i"), FMath::Abs(GetFractionTicks())); break;
			case TCHAR('n'): Result += FString::Printf(TEXT("%09i"), FMath::Abs(GetFractionNano())); break;
			default:
				Result += *Format;
			}
		}
		else
		{
			Result += *Format;
		}

		++Format;
	}

	return Result;
}


/* FTimespan static interface
 *****************************************************************************/

bool FTimespan::Parse(const FString& TimespanString, FTimespan& OutTimespan)
{
	// @todo gmp: implement stricter FTimespan parsing; this implementation is too forgiving

	// get string tokens
	const bool HasFractional = TimespanString.Contains(TEXT(".")) || TimespanString.Contains(TEXT(","));
	FString TokenString = TimespanString;
	TokenString.ReplaceInline(TEXT("."), TEXT(":"));
	TokenString.ReplaceInline(TEXT(","), TEXT(":"));

	const bool Negative = TokenString.StartsWith(TEXT("-"));
	TokenString.ReplaceInline(TEXT("-"), TEXT(":"));
	TokenString.ReplaceInline(TEXT("+"), TEXT(":"));

	TArray<FString> Tokens;
	TokenString.ParseIntoArray(Tokens, TEXT(":"), true);

	if (!HasFractional)
	{
		Tokens.AddDefaulted();
	}

	// token verification
	for (const FString& Token : Tokens)
	{
		if (!Token.IsEmpty() && !Token.IsNumeric())
		{
			return false;
		}
	}

	// add missing tokens
	if (Tokens.Num() < 5)
	{
		Tokens.InsertDefaulted(0, 5 - Tokens.Num());
	}
	else if (Tokens.Num() > 5)
	{
		return false;
	}

	// pad fractional token with zeros
	if (HasFractional)
	{
		const int32 FractionalLen = Tokens[4].Len();

		if (FractionalLen > 9)
		{
			Tokens[4].LeftInline(9, EAllowShrinking::No);
		}
		else if (FractionalLen < 9)
		{
			Tokens[4] += FString("000000000").Left(9 - FractionalLen);
		}
	}

	const int32 Days = FCString::Atoi(*Tokens[0]);
	const int32 Hours = FCString::Atoi(*Tokens[1]);
	const int32 Minutes = FCString::Atoi(*Tokens[2]);
	const int32 Seconds = FCString::Atoi(*Tokens[3]);
	const int32 FractionNano = FCString::Atoi(*Tokens[4]);

	if ((Days > (ETimespan::MaxTicks / ETimespan::TicksPerDay) - 1))
	{
		return false;
	}

	if ((Hours > 23) || (Minutes > 59) || (Seconds > 59) || (FractionNano > 999999999))
	{
		return false;
	}

	OutTimespan.Assign(Days, Hours, Minutes, Seconds, FractionNano);

	if (Negative)
	{
		OutTimespan.Ticks *= -1;
	}

	return true;
}


/* FTimespan friend functions
 *****************************************************************************/

FArchive& operator<<(FArchive& Ar, FTimespan& Timespan)
{
	return Ar << Timespan.Ticks;
}


uint32 GetTypeHash(const FTimespan& Timespan)
{
	return GetTypeHash(Timespan.Ticks);
}


/* FTimespan implementation
 *****************************************************************************/

void FTimespan::Assign(int32 Days, int32 Hours, int32 Minutes, int32 Seconds, int32 FractionNano)
{
	int64 TotalTicks = 0;

	TotalTicks += Days * ETimespan::TicksPerDay;
	TotalTicks += Hours * ETimespan::TicksPerHour;
	TotalTicks += Minutes * ETimespan::TicksPerMinute;
	TotalTicks += Seconds * ETimespan::TicksPerSecond;
	TotalTicks += FractionNano / ETimespan::NanosecondsPerTick;

	check((TotalTicks >= ETimespan::MinTicks) && (TotalTicks <= ETimespan::MaxTicks));

	Ticks = TotalTicks;
}
