// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/TrackedActivity.h"
#include "AutoRTFM.h"
#include "Containers/UnrealString.h"
#include "Misc/ScopeLock.h"
#include "Misc/StringBuilder.h"
#include "Misc/TransactionallySafeCriticalSection.h"

// Singleton class that keep track of all the existing activities and their stacks
class FTrackedActivityManager
{
public:
	static FTrackedActivityManager& Get()
	{
		static FTrackedActivityManager Manager;
		return Manager;
	}

	struct FActivity;
	using ELight = FTrackedActivity::ELight;
	using EType = FTrackedActivity::EType;

	struct FInfo
	{
		FString Status;
		ELight Light;
		bool bShowParent;
	};

	struct FActivity
	{
		FString Name;
		TArray<FInfo> Stack;
		EType Type;
		int32 SortValue;
		uint32 Id;
	};

	FTransactionallySafeCriticalSection ActivitiesCs;
	TArray<FActivity*> Activities;
	TUniqueFunction<void(FTrackedActivity::EEvent Event, const FTrackedActivity::FInfo& Info)> EventFunc;
	uint32 EventFuncMaxDepth = ~0u;
	uint32 IdCounter = 1u;

	FActivity* Create(const TCHAR* Name, const TCHAR* Status, ELight Light, EType Type, int32 SortValue)
	{
		FActivity* Activity = new FActivity();
		Activity->Name = Name;
		Activity->Type = Type;
		Activity->SortValue = SortValue;
		Activity->Stack.Add({ Status, Light, false });

		UE::TScopeLock _(ActivitiesCs);
		Activity->Id = IdCounter++;
		Activities.Add(Activity);
		SendEvent(FTrackedActivity::EEvent::Added, *Activity);
		return Activity;
	}

	void Destroy(FActivity* Activity)
	{
		check(Activity->Stack.Num() == 1);
		{
			UE::TScopeLock _(ActivitiesCs);
			Activities.Remove(Activity);
			SendEvent(FTrackedActivity::EEvent::Removed, *Activity);
		}
		delete Activity;
	}

	uint32 Push(FActivity& Activity, const TCHAR* Status, bool bShowParent, ELight Light)
	{
		UE::TScopeLock _(ActivitiesCs);
		uint32 Index = Activity.Stack.Num();
		Activity.Stack.Add({ Status, Light, bShowParent });
		SendEvent(FTrackedActivity::EEvent::Changed, Activity);
		return Index;
	}

	void Pop(FActivity& Activity)
	{
		UE::TScopeLock _(ActivitiesCs);
		Activity.Stack.SetNum(Activity.Stack.Num() - 1);
		SendEvent(FTrackedActivity::EEvent::Changed, Activity);
	}

	void Update(FActivity& Activity, const TCHAR* Status, uint32 Index)
	{
		UE::TScopeLock _(ActivitiesCs);
		FTrackedActivityManager::FInfo& Info = Index == ~0u ? Activity.Stack.Last() : Activity.Stack[Index];
		Info.Status = Status;
		SendEvent(FTrackedActivity::EEvent::Changed, Activity);
	}

	void Update(FActivity& Activity, const TCHAR* Status, ELight Light, uint32 Index)
	{
		UE::TScopeLock _(ActivitiesCs);
		FTrackedActivityManager::FInfo& Info = Index == ~0u ? Activity.Stack.Last() : Activity.Stack[Index];
		Info.Status = Status;
		Info.Light = Light;
		SendEvent(FTrackedActivity::EEvent::Changed, Activity);
	}

	void Update(FActivity& Activity, ELight Light, uint32 Index)
	{
		UE::TScopeLock _(ActivitiesCs);
		FTrackedActivityManager::FInfo& Info = Index == ~0u ? Activity.Stack.Last() : Activity.Stack[Index];
		Info.Light = Light;
		SendEvent(FTrackedActivity::EEvent::Changed, Activity);
	}

	FTrackedActivity::FInfo GetInfo(const FActivity& Activity, uint32 MaxDepth, TStringBuilder<256>& Temp)
	{
		uint32 StackIndex = FMath::Min<uint32>(Activity.Stack.Num(), MaxDepth) - 1;
		auto& Info = Activity.Stack[StackIndex];

		const TCHAR* Status = *Info.Status;
		ELight Light = Info.Light;

		if (Info.bShowParent)
		{
			Temp.Append(Activity.Stack[StackIndex - 1].Status);
			Temp.Append(Status);
			Status = *Temp;
		}

		while (Light == ELight::Inherit)
		{
			if (StackIndex == 0)
			{
				Light = ELight::None;
				break;
			}
			Light = Activity.Stack[--StackIndex].Light;
		}

		return { *Activity.Name, Status, Light, Activity.Type, Activity.SortValue, Activity.Id };
	}

	void TraverseActivities(const TFunction<void(const FTrackedActivity::FInfo& Info)>& Func)
	{
		UE::TScopeLock _(ActivitiesCs);

		for (auto A : Activities)
		{
			TStringBuilder<256> Temp;
			Func(GetInfo(*A, ~0u, Temp));
		}
	}

	void RegisterEventListener(TUniqueFunction<void(FTrackedActivity::EEvent Event, const FTrackedActivity::FInfo& Info)>&& Func, uint32 MaxDepth)
	{
		checkf(!EventFunc, TEXT("TrackedActivity system is only supporting one event listener at the time. Please add support"));
		UE::TScopeLock _(ActivitiesCs);

		EventFunc = MoveTemp(Func);
		EventFuncMaxDepth = MaxDepth;
	}

	void UnregisterEventListener()
	{
		checkf(EventFunc, TEXT("TrackedActivity system is not currently registered"));
		UE::TScopeLock _(ActivitiesCs);

		EventFunc = {};
	}

	void SendEvent(FTrackedActivity::EEvent Event, FActivity& Activity)
	{
		if (!EventFunc)
			return;
		TStringBuilder<256> Temp;
		FTrackedActivity::FInfo ActivityInfo{GetInfo(Activity, EventFuncMaxDepth, Temp)};
		
		// This callback mechanism is used to update the console output windows. These are wildly AutoRTFM-unsafe
		// (since they are manipulating HWNDs directly, etc.), so we always run the callback in the open.
		UE_AUTORTFM_OPEN
		{
			EventFunc(Event, ActivityInfo);
		};
	}
};


FTrackedActivity::FTrackedActivity(const TCHAR* Name, const TCHAR* Status, ELight Light, EType Type, int32 SortValue)
{
	Internal = FTrackedActivityManager::Get().Create(Name, Status, Light, Type, SortValue);
}

FTrackedActivity::~FTrackedActivity()
{
	FTrackedActivityManager::Get().Destroy((FTrackedActivityManager::FActivity*)Internal);
}

uint32 FTrackedActivity::Push(const TCHAR* Status, bool bShowParent, ELight Light)
{
	return FTrackedActivityManager::Get().Push(*(FTrackedActivityManager::FActivity*)Internal, Status, bShowParent, Light);
}

void FTrackedActivity::Pop()
{
	FTrackedActivityManager::Get().Pop(*(FTrackedActivityManager::FActivity*)Internal);
}

void FTrackedActivity::Update(const TCHAR* Status, uint32 Index)
{
	FTrackedActivityManager::Get().Update(*(FTrackedActivityManager::FActivity*)Internal, Status, Index);
}

void FTrackedActivity::Update(const TCHAR* Status, ELight Light, uint32 Index)
{
	FTrackedActivityManager::Get().Update(*(FTrackedActivityManager::FActivity*)Internal, Status, Light, Index);
}

void FTrackedActivity::Update(ELight Light, uint32 Index)
{
	FTrackedActivityManager::Get().Update(*(FTrackedActivityManager::FActivity*)Internal, Light, Index);
}

FTrackedActivity& FTrackedActivity::GetEngineActivity()
{
	static TSharedPtr<FTrackedActivity> A(MakeShared<FTrackedActivity>(TEXT("Status"), TEXT("Unknown"), ELight::None, EType::Activity, 0));
	return *A;
}

FTrackedActivity& FTrackedActivity::GetIOActivity()
{
	static TSharedPtr<FTrackedActivity> A(MakeShared<FTrackedActivity>(TEXT("I/O"), TEXT("Idle"), ELight::None, EType::Activity, 1));
	return *A;
}

void FTrackedActivity::RegisterEventListener(TUniqueFunction<void(EEvent Event, const FInfo& Info)>&& Func, uint32 MaxDepth)
{
	FTrackedActivityManager::Get().RegisterEventListener(MoveTemp(Func), MaxDepth);
}

void FTrackedActivity::UnregisterEventListener()
{
	FTrackedActivityManager::Get().UnregisterEventListener();
}

void FTrackedActivity::TraverseActivities(const TFunction<void(const FInfo& Info)>& Func)
{
	FTrackedActivityManager::Get().TraverseActivities(Func);
}

FTrackedActivityScope::FTrackedActivityScope(FTrackedActivity& A, const TCHAR* Status, bool bShowParent, FTrackedActivity::ELight Light)
:	Activity(A)
{
	Activity.Push(Status, bShowParent, Light);
}
FTrackedActivityScope::~FTrackedActivityScope()
{
	Activity.Pop();
}
