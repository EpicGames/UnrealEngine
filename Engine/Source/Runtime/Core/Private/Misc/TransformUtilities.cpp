// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/TransformUtilities.h"
#include "Misc/Crc.h"
#include "Math/UnrealMathUtility.h"
#include "Math/Vector.h"
#include "Math/Transform.h"

uint32 TransformUtilities::GetRoundedTransformCRC32(const FTransform& InTransform)
{
	FVector Location(InTransform.GetLocation());
	// Include transform - round sufficiently to ensure stability
	FIntVector IntLocation(FMath::RoundToInt32(Location.X), FMath::RoundToInt32(Location.Y), FMath::RoundToInt32(Location.Z));
	uint32 CRC = FCrc::TypeCrc32(IntLocation);

	FRotator Rotator(InTransform.Rotator().GetDenormalized());
	const int32 MAX_DEGREES = 360;
	FIntVector IntRotation(FMath::RoundToInt32(Rotator.Pitch) % MAX_DEGREES, FMath::RoundToInt32(Rotator.Yaw) % MAX_DEGREES, FMath::RoundToInt32(Rotator.Roll) % MAX_DEGREES);
	CRC = FCrc::TypeCrc32(IntRotation, CRC);

	const float SCALE_FACTOR = 100;
	FVector Scale(InTransform.GetScale3D());
	FIntVector IntScale(FMath::RoundToInt32(Scale.X * SCALE_FACTOR), FMath::RoundToInt32(Scale.Y * SCALE_FACTOR), FMath::RoundToInt32(Scale.Z * SCALE_FACTOR));
	return FCrc::TypeCrc32(IntScale, CRC);
}


