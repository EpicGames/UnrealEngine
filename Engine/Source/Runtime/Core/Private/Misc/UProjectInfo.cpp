// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/UProjectInfo.h"
#include "Logging/LogMacros.h"
#include "HAL/FileManager.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"

DECLARE_LOG_CATEGORY_EXTERN(LogUProjectInfo, Log, All);
DEFINE_LOG_CATEGORY(LogUProjectInfo);

FUProjectDictionary::FUProjectDictionary(const FString& InRootDir)
{
	RootDir = InRootDir;
	Refresh();
}

void FUProjectDictionary::Refresh()
{
	ProjectRootDirs.Reset();
	ShortProjectNameDictionary.Reset();

	// Find all the .uprojectdirs files contained in the root folder
	TArray<FString> ProjectDirsFiles;
	IFileManager::Get().FindFiles(ProjectDirsFiles, *(RootDir / FString("*.uprojectdirs")), true, false);

	// Get the normalized path to the root directory
	FString NormalizedRootDir = FPaths::ConvertRelativePathToFull(RootDir);
	FPaths::NormalizeDirectoryName(NormalizedRootDir);
	FString NormalizedRootDirPrefix = NormalizedRootDir / TEXT("");

	// Add all the project root directories to ProjectRootDirs
	for(const FString& ProjectDirsFile: ProjectDirsFiles)
	{
		TArray<FString> Lines;
		if(FFileHelper::LoadANSITextFileToStrings(*(RootDir / ProjectDirsFile), &IFileManager::Get(), Lines))
		{
			for(const FString& Line: Lines)
			{
				FString Entry = Line.TrimStart();
				if(!Entry.IsEmpty() && !Entry.StartsWith(";"))
				{
					FString DirectoryName = FPaths::ConvertRelativePathToFull(RootDir, Entry);
					FPaths::NormalizeDirectoryName(DirectoryName);
					if (DirectoryName.StartsWith(NormalizedRootDirPrefix) || DirectoryName == NormalizedRootDir)
					{
						ProjectRootDirs.AddUnique(DirectoryName);
					}
					else
					{
						UE_LOGF(LogUProjectInfo, Warning, "Project search path '%ls' is not under root directory, ignoring.", *Entry);
					}
				}
			}
		}
	}

	// Search for all the projects under each root directory
	for(const FString& ProjectRootDir: ProjectRootDirs)
	{
		// Enumerate the subdirectories
		TArray<FString> ProjectDirs;
		IFileManager::Get().FindFiles(ProjectDirs, *(ProjectRootDir / FString("*")), false, true);

		// Check each one for project files
		for(const FString& ProjectDir: ProjectDirs)
		{
			// Enumerate all the project files
			TArray<FString> ProjectFiles;
			IFileManager::Get().FindFiles(ProjectFiles, *(ProjectRootDir / ProjectDir / TEXT("*.uproject")), true, false);

			// Add all the projects to the dictionary
			for(const FString& ProjectFile: ProjectFiles)
			{
				FString ShortName = FPaths::GetBaseFilename(ProjectFile);
				FString FullProjectFile = ProjectRootDir / ProjectDir / ProjectFile;
				ShortProjectNameDictionary.Add(ShortName, FullProjectFile);
			}
		}
	}
}

bool FUProjectDictionary::IsForeignProject(const FString& InProjectFileName) const
{
	FString ProjectFileName = FPaths::ConvertRelativePathToFull(InProjectFileName);

	// Check if it's already in the project dictionary
	for(TMap<FString, FString>::TConstIterator Iter(ShortProjectNameDictionary); Iter; ++Iter)
	{
		if(Iter.Value() == ProjectFileName)
		{
			return false;
		}
	}

	// If not, it may be a new project. Check if its parent directory is a project root dir.
	FString ProjectRootDir = FPaths::GetPath(FPaths::GetPath(ProjectFileName));
	if(ProjectRootDirs.Contains(ProjectRootDir))
	{
		return false;
	}

	// Otherwise it's a foreign project
	return true;
}

FString FUProjectDictionary::GetRelativeProjectPathForGame(const TCHAR* InGameName, const FString& BaseDir) const
{
	const FString* ProjectFile = ShortProjectNameDictionary.Find(*(FString(InGameName).ToLower()));
	if (ProjectFile != NULL)
	{
		FString RelativePath = *ProjectFile;
		FPaths::MakePathRelativeTo(RelativePath, *BaseDir);
		return RelativePath;
	}
	return TEXT("");
}

FString FUProjectDictionary::GetProjectPathForGame(const TCHAR* InGameName) const
{
	if (const FString* ProjectFile = ShortProjectNameDictionary.Find(*(FString(InGameName).ToLower())))
	{
		return *ProjectFile;
	}
	return {};
}

TArray<FString> FUProjectDictionary::GetProjectPaths() const
{
	TArray<FString> Paths;
	ShortProjectNameDictionary.GenerateValueArray(Paths);
	return Paths;
}

FUProjectDictionary& FUProjectDictionary::GetDefault()
{
	static FUProjectDictionary DefaultDictionary(FPaths::RootDir());

#if !NO_LOGGING
	static bool bHaveLoggedProjects = false;
	if(!bHaveLoggedProjects)
	{
		UE_LOGF(LogUProjectInfo, Verbose, "Found projects:");
		for(TMap<FString, FString>::TConstIterator Iter(DefaultDictionary.ShortProjectNameDictionary); Iter; ++Iter)
		{
			UE_LOGF(LogUProjectInfo, Verbose, "    %ls: \"%ls\"", *Iter.Key(), *Iter.Value());
		}
		bHaveLoggedProjects = true;
	}
#endif

	return DefaultDictionary;
}
