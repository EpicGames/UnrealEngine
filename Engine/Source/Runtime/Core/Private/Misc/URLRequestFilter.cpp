// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/URLRequestFilter.h"
#include "Algo/AnyOf.h"
#include "Algo/Find.h"
#include "Algo/ForEach.h"
#include "Containers/Array.h"
#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include "CoreGlobals.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/Parse.h"
#include "String/Find.h"

namespace UE::Core
{

FURLRequestFilter::FURLRequestFilter(const TCHAR* ConfigSectionRootName, const FString& ConfigFileName)
{
	UpdateConfig(ConfigSectionRootName, ConfigFileName);
}

FURLRequestFilter::FURLRequestFilter(const FRequestMap& InAllowedRequests)
	: AllowedRequests(InAllowedRequests)
{
}

FURLRequestFilter::FURLRequestFilter(FRequestMap&& InAllowedRequests)
	: AllowedRequests(MoveTemp(InAllowedRequests))
{
}

void FURLRequestFilter::UpdateConfig(const TCHAR* ConfigSectionRootName, const FString& ConfigFileName)
{
	if (ConfigFileName.IsEmpty() || GConfig == nullptr)
	{
		return;
	}

	AllowedRequests.Empty();

	const FConfigFile* File = GConfig->FindConfigFile(ConfigFileName);
	if (File != nullptr)
	{
		Algo::ForEach(*File, [this, ConfigSectionRootName](const FConfigFileMap::ElementType& Entry)
			{
				const FString& SectionName = Entry.Key;

				// Find config sections with the syntax [ConfigSectionRootName scheme]
				const int32 DelimiterIndex = UE::String::FindLastChar(SectionName, ' ');
				if (DelimiterIndex != INDEX_NONE)
				{
					FStringView RootName = SectionName;
					RootName.LeftInline(DelimiterIndex);
					RootName.TrimStartAndEndInline();

					if (!RootName.Equals(ConfigSectionRootName, ESearchCase::CaseSensitive))
					{
						return;
					}

					FStringView Scheme = SectionName;
					Scheme.RightChopInline(DelimiterIndex);
					Scheme.TrimStartAndEndInline();

					TArray<FString> Domains;
					Entry.Value.MultiFind(TEXT("AllowedDomains"), Domains, true);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
					TArray<FString> DevDomains;
					Entry.Value.MultiFind(TEXT("DevAllowedDomains"), DevDomains, true);
					Domains.Append(DevDomains);
#endif

					// Empty scheme or domain list is OK
					AllowedRequests.Emplace(FString(Scheme), MoveTemp(Domains));
				}
				else if (SectionName.Equals(ConfigSectionRootName, ESearchCase::CaseSensitive))
				{
					AllowedRequests.Emplace(FString(), TArray<FString>());
				}
			});
	}
}

bool FURLRequestFilter::IsRequestAllowed(FStringView InURL) const
{
	if (AllowedRequests.IsEmpty())
	{
		// No filters configured
		return true;
	}

	FStringView Scheme;
	FStringView Domain;

	const FStringView Delimiter = TEXT("://");
	const int32 FoundIndex = UE::String::FindFirst(InURL, Delimiter);
	if (FoundIndex != INDEX_NONE)
	{
		Scheme = InURL.Left(FoundIndex);
		Domain = InURL.RightChop(FoundIndex + Delimiter.Len());
	}
	else
	{
		Domain = InURL;
	}

	// userinfo is unsupported (usernames in URLs, like "user" in http://user@epicgames.com).

	// Check for an IPv6 address delimited in '[' and ']'
	if (Domain.StartsWith('['))
	{
		const int32 DomainEnd = UE::String::FindFirstChar(Domain, ']');
		if (DomainEnd != INDEX_NONE)
		{
			Domain.LeftInline(DomainEnd + 1);
		}
	}
	else
	{
		const int32 FoundDomainEndIndex = UE::String::FindFirstOfAnyChar(Domain, TEXTVIEW("/?:#"));
		if (FoundDomainEndIndex != INDEX_NONE)
		{
			Domain.LeftInline(FoundDomainEndIndex);
		}
	}

	// Look for the scheme, they are case-insensitive
	const FRequestMap::ElementType* FoundScheme = Algo::FindByPredicate(AllowedRequests, [Scheme](const FRequestMap::ElementType& Item)
	{
		return !Item.Key.IsEmpty() && Scheme.Equals(Item.Key, ESearchCase::IgnoreCase);
	});

	if (FoundScheme == nullptr)
	{
		UE_LOGF(LogCore, Log, "Scheme for URL %.*ls not allowed.", InURL.Len(), InURL.GetData());
		return false;
	}

	if (FoundScheme->Value.IsEmpty())
	{
		// No domain filters configured for this scheme
		return true;
	}

	// Look for the domain in this scheme's filter, they are case-insensitive
	const bool bFoundDomain = Algo::AnyOf(FoundScheme->Value, [Domain](const FString& Entry)
	{
		if (UE::String::FindFirstChar(Entry, '.') == 0)
		{
			// Entry starts with a period, check the end of the domain to allow for subdomains
			return Domain.EndsWith(Entry, ESearchCase::IgnoreCase);
		}
		else
		{
			// Entry does not start with a period, match the whole domain.
			return !Entry.IsEmpty() && Domain.Equals(Entry, ESearchCase::IgnoreCase);
		}
	});

	UE_CLOGF(!bFoundDomain, LogCore, Log, "Domain for URL %.*ls not allowed.", InURL.Len(), InURL.GetData());

	return bFoundDomain;
}

}