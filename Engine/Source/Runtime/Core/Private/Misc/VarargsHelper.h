// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/CString.h"
#include "HAL/UnrealMemory.h"

#include <cstdarg>

template <typename SerializeFuncType>
bool GrowableLogfV(const TCHAR* Fmt, va_list Args, const SerializeFuncType& SerializeFunc)
{
	// Allocate some stack space to use on the first pass. This should be sufficient for most strings.
	constexpr int DefaultBufferSize = 512;

	TCHAR	StackBuffer[DefaultBufferSize];
	TCHAR*	Buffer = StackBuffer;
	int32	BufferSize = DefaultBufferSize;
	TCHAR*	AllocatedBuffer = nullptr;
	int32	Length = -1;

	for (;;)
	{
		va_list ArgsCopy;
		va_copy(ArgsCopy, Args);
		Length = FCString::GetVarArgs(Buffer, BufferSize, Fmt, ArgsCopy);
		va_end(ArgsCopy);

		if (Length >= 0 && Length < BufferSize - 1)
		{
			break;
		}

		// Make increasingly large allocations on the heap until we can hold the formatted string.
		// We need to use SystemMalloc here, as GMalloc might not be safe.
		FMemory::SystemFree(AllocatedBuffer);
		BufferSize *= 2;
		Buffer = AllocatedBuffer = (TCHAR*) FMemory::SystemMalloc(BufferSize * sizeof(TCHAR));
		if (Buffer == nullptr)
		{
			return false;
		}
	}
	Buffer[Length] = TEXT('\0');

	SerializeFunc(Buffer);
	FMemory::SystemFree(AllocatedBuffer);
	return true;
}

// This macro expects to be used in a variadic function where the parameter list ends with
// a `Fmt` text string, followed by the `...` variadic indicator. The passed-in code snippet
// will have access to the resulting string via a variable named `Buffer`.
#define GROWABLE_LOGF(SerializeSnippet) \
	{ \
		va_list Args; \
		va_start(Args, Fmt); \
		GrowableLogfV(Fmt, Args, [&](TCHAR* Buffer) { SerializeSnippet; }); \
		va_end(Args); \
	}
