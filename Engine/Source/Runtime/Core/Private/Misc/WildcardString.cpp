// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/WildcardString.h"

#include "Misc/Char.h"

/* FWildcardString static functions
 *****************************************************************************/

bool FWildcardString::ContainsWildcards(const TCHAR* Pattern)
{
	if (Pattern != nullptr)
	{
		while (*Pattern != EndOfString)
		{
			if ((*Pattern == ExactWildcard) || (*Pattern == SequenceWildcard))
			{
				return true;
			}

			Pattern++;
		}
	}
	
	return false;
}


bool FWildcardString::IsMatch(const TCHAR* Pattern, const TCHAR* Input)
{
	if ((Pattern == nullptr) || (Input == nullptr))
	{
		return false;
	}

	const TCHAR* pp = Pattern;
	const TCHAR* pinput = nullptr;

	while (*Input != EndOfString)
	{
		if (*Pattern == SequenceWildcard)
		{
			// exit early
			if (*++Pattern == EndOfString)
			{
				return true;
			}

			pp = Pattern;
			pinput = Input + 1;
		}
		else if ((*Pattern == ExactWildcard) || (*Pattern == *Input))
		{
			// single match
			Pattern++;
			Input++;
		}
		else if (pinput == nullptr)
		{
			// single mismatch
			return false;
		}
		else
		{
			Pattern = pp;
			Input = pinput++;
		}
	}

	// skip trailing sequence wildcards
	while (*Pattern == SequenceWildcard)
	{
		Pattern++;
	}

	// matching complete
	if (*Pattern == EndOfString)
	{
		return true;
	}

	// matching incomplete
	return false;
}

bool FWildcardString::IsMatchSubstring(const TCHAR* Pattern, const TCHAR* Input, const TCHAR* InputEnd, ESearchCase::Type SearchCase)
{
	if ((Pattern == nullptr) || (Input == nullptr) || (InputEnd == nullptr))
	{
		return false;
	}

	const TCHAR* pp = Pattern;
	const TCHAR* pinput = nullptr;

	while (Input != InputEnd)
	{
		TCHAR InputChar = SearchCase == ESearchCase::IgnoreCase ? TChar<TCHAR>::ToLower(*Input) : *Input;
		TCHAR PatternChar = SearchCase == ESearchCase::IgnoreCase ? TChar<TCHAR>::ToLower(*Pattern) : *Pattern;

		if (*Pattern == SequenceWildcard)
		{
			// exit early
			if (*++Pattern == EndOfString)
			{
				return true;
			}

			pp = Pattern;
			pinput = Input + 1;
		}
		else if ((*Pattern == ExactWildcard) || (PatternChar == InputChar))
		{
			// single match
			Pattern++;
			Input++;
		}
		else if (pinput == nullptr)
		{
			// single mismatch
			return false;
		}
		else
		{
			Pattern = pp;
			Input = pinput++;
		}
	}

	// skip trailing sequence wildcards
	while (*Pattern == SequenceWildcard)
	{
		Pattern++;
	}

	// matching complete
	if (*Pattern == EndOfString)
	{
		return true;
	}

	// matching incomplete
	return false;
}
