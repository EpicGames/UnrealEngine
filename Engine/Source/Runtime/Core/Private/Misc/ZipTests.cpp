// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"

#include "Algo/Sort.h"
#include "Containers/AnsiString.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Misc/AutomationTest.h"
#include "Misc/Zip.h"

#include <type_traits>

#if WITH_DEV_AUTOMATION_TESTS

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FUEZipTests, "System.Core.UEZip", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool FUEZipTests::RunTest(const FString& Parameters)
{
	const TArray<int32>   Arr1 = { 1, 2, 3, 4, 5 };
	const TArray<FString> Arr2 = { "Orange", "Banana", "Apple", "Pear", "Cherry" };
	const float           Arr3[] = { 3.1415f, 1.4142f, 2.7183f, 1.6180f, 0.1234f };

	// Test basic zipped iteration
	{
		// Single
		{
			TArray<int32> Copy;
			for (const auto& Single : UE::Zip(Arr1))
			{
				static_assert(std::is_same_v<decltype(Single), const TTuple<const int32&>&>);
				Copy.Add(Single.Get<0>());
			}
			check(Copy == TArray<int32>({ 1, 2, 3, 4, 5 }));
		}

		// Double
		{
			TArray<int32>   Copy1;
			TArray<FString> Copy2;
			for (const auto& Pair : UE::Zip(Arr1, Arr2))
			{
				static_assert(std::is_same_v<decltype(Pair), const TTuple<const int32&, const FString&>&>);
				Copy1.Add(Pair.Get<0>());
				Copy2.Add(Pair.Get<1>());
			}
			check(Copy1 == TArray<int32>  ({ 1, 2, 3, 4, 5 }));
			check(Copy2 == TArray<FString>({ "Orange", "Banana", "Apple", "Pear", "Cherry" }));
		}

		// Triple, with a different container type
		{
			TArray<int32>   Copy1;
			TArray<FString> Copy2;
			TArray<float>   Copy3;
			for (const auto& Triple : UE::Zip(Arr1, Arr2, Arr3))
			{
				static_assert(std::is_same_v<decltype(Triple), const TTuple<const int32&, const FString&, const float&>&>);
				Copy1.Add(Triple.Get<0>());
				Copy2.Add(Triple.Get<1>());
				Copy3.Add(Triple.Get<2>());
			}
			check(Copy1 == TArray<int32>  ({ 1, 2, 3, 4, 5 }));
			check(Copy2 == TArray<FString>({ "Orange", "Banana", "Apple", "Pear", "Cherry" }));
			check(Copy3 == TArray<float>  ({ 3.1415f, 1.4142f, 2.7183f, 1.6180f, 0.1234f }));
		}

		// Mutate container elements
		{
			TArray<int32>   Arr1Copy = Arr1;
			TArray<FString> Arr2Copy = Arr2;
			for (const auto& Pair : UE::Zip(Arr1Copy, Arr2Copy))
			{
				static_assert(std::is_same_v<decltype(Pair), const TTuple<int32&, FString&>&>);

				// Scale int by 10
				Pair.Get<0>() *= 10;

				// Append a copy of the string separated by |
				Pair.Get<1>() += FString::Printf(TEXT("|%s"), *Pair.Get<1>());
			}
			check(Arr1Copy == TArray<int32>  ({ 10, 20, 30, 40, 50 }));
			check(Arr2Copy == TArray<FString>({ "Orange|Orange", "Banana|Banana", "Apple|Apple", "Pear|Pear", "Cherry|Cherry" }));
		}

		// Mix constness
		{
			TArray<int32>   Arr1Copy = Arr1;
			TArray<FString> Arr2Copy = Arr2;
			for (const auto& Pair : UE::Zip(Arr1Copy, AsConst(Arr2Copy)))
			{
				static_assert(std::is_same_v<decltype(Pair), const TTuple<int32&, const FString&>&>);
			}
		}

		// Mix lengths, and test structured binding
		{
			TArray<int32> Short1 = { 2, 3, 5 };
			TArray<float> Short3 = { 4.0f, 3.0f, 2.0f, 1.0f };

			using TupleType = TTuple<int32, FString, float>;
			using CopyType  = TArray<TupleType>;
			CopyType Copy;

			// Should only copy as many arguments as Short1 has
			for (auto&& [Int, String, Float] : UE::Zip(Short1, Arr2, Short3))
			{
				Copy.Add({ Int, String, Float });
			}
			check(Copy == CopyType({ { 2, "Orange", 4.0f }, { 3, "Banana", 3.0f }, { 5, "Apple", 2.0f } }));
		}

		// Mix lvalues and rvalues
		{
			auto MakeArray = []()
			{
				return TArray<FAnsiString>{ "Abcde", "Fghi", "Klm", "No", "P" };
			};

			TArray<int32>       Copy1;
			TArray<FAnsiString> Copy2;
			for (const auto& Pair : UE::Zip(Arr1, MakeArray()))
			{
				static_assert(std::is_same_v<decltype(Pair), const TTuple<const int32&, const FAnsiString&>&>);
				Copy1.Add(Pair.Key);
				Copy2.Add(Pair.Value);
			}
			check(Copy1 == TArray<int32>({ 1, 2, 3, 4, 5 }));
			check(Copy2 == TArray<FAnsiString>({ "Abcde", "Fghi", "Klm", "No", "P" }));
		}
	}

	// Test zipped sorting
	{
		// Sort by the strings
		TArray<int32>   Arr1Copy = Arr1;
		TArray<FString> Arr2Copy = Arr2;
		Algo::SortBy(UE::Zip(Arr1Copy, Arr2Copy), GetElementByIndex<1>);
		check(Arr1Copy == TArray<int32>  ({ 3, 2, 5, 1, 4 }));
		check(Arr2Copy == TArray<FString>({ "Apple", "Banana", "Cherry", "Orange", "Pear" }));

		// Sort by the ints
		Algo::SortBy(UE::Zip(Arr1Copy, Arr2Copy), GetElementByIndex<0>);
		check(Arr1Copy == TArray<int32>({ 1, 2, 3, 4, 5 }));
		check(Arr2Copy == TArray<FString>({ "Orange", "Banana", "Apple", "Pear", "Cherry" }));
	}

	return !HasAnyErrors();
}

#endif // WITH_DEV_AUTOMATION_TESTS
