// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/BuildVersion.h"
#include "Misc/FileHelper.h"
#include "Misc/App.h"
#include "Modules/ModuleManager.h"
#include "Modules/SimpleParse.h"
#include "Misc/Paths.h"

FBuildVersion::FBuildVersion()
	: MajorVersion(0)
	, MinorVersion(0)
	, PatchVersion(0)
	, Changelist(0)
	, CompatibleChangelist(0)
	, IsLicenseeVersion(0)
	, IsPromotedBuild(0)
{
}

int FBuildVersion::GetEffectiveCompatibleChangelist() const
{
	return (Changelist != 0 && CompatibleChangelist != 0) ? CompatibleChangelist : Changelist;
}

FEngineVersion FBuildVersion::GetEngineVersion() const
{
	int EncodedChangelist = Changelist;
	if (IsLicenseeVersion)
	{
		EncodedChangelist = FEngineVersion::EncodeLicenseeChangelist(EncodedChangelist);
	}
	return FEngineVersion((uint16)MajorVersion, (uint16)MinorVersion, (uint16)PatchVersion, EncodedChangelist, BranchName);
}

FEngineVersion FBuildVersion::GetCompatibleEngineVersion() const
{
	int EncodedChangelist = GetEffectiveCompatibleChangelist();
	if (IsLicenseeVersion)
	{
		EncodedChangelist = FEngineVersion::EncodeLicenseeChangelist(EncodedChangelist);
	}
	return FEngineVersion((uint16)MajorVersion, (uint16)MinorVersion, (uint16)PatchVersion, EncodedChangelist, BranchName);
}

FString FBuildVersion::GetDefaultFileName()
{
	return FPaths::EngineDir() / TEXT("Build/Build.version");
}

FString FBuildVersion::GetFileNameForCurrentExecutable()
{
	FString AppExecutableName = FPlatformProcess::ExecutableName();
#if PLATFORM_DESKTOP
	if (AppExecutableName.EndsWith(TEXT("-Cmd")))
	{
		AppExecutableName.LeftChopInline(4, EAllowShrinking::No);
	}
#endif
#if UE_BUILD_DEVELOPMENT
	// For DebugGame builds we want to read the development version file because it's in an engine folder
	if (FApp::GetBuildConfiguration() == EBuildConfiguration::DebugGame)
	{
		for (int32 Idx = 0; Idx < 2; Idx++)
		{
			int EndIdx;
			if (!AppExecutableName.FindLastChar('-', EndIdx))
			{
				break;
			}
			AppExecutableName.LeftInline(EndIdx, EAllowShrinking::No);
		}
	}
#endif
	FString Result = FPlatformProcess::GetModulesDirectory() / AppExecutableName + TEXT(".version");
	return Result;
}

bool FBuildVersion::TryRead(const FString& FileName, FBuildVersion& OutVersion)
{
	// Read the file to a string
	FString Text;
	if (!FFileHelper::LoadFileToString(Text, *FileName))
	{
		return false;
	}

	return TryReadFromString(Text, OutVersion);
}

bool FBuildVersion::TryReadFromString(const FString& Text, FBuildVersion& OutVersion)
{
	const TCHAR* Ptr = *Text;
	if (!FSimpleParse::MatchZeroOrMoreWhitespace(Ptr) || !FSimpleParse::MatchChar(Ptr, TEXT('{')) || !FSimpleParse::MatchZeroOrMoreWhitespace(Ptr) || FSimpleParse::MatchChar(Ptr, TEXT('}')))
	{
		return false;
	}

	bool bParsedMajorVersion = false;
	bool bParsedMinorVersion = false;
	bool bParsedPatchVersion = false;
	for (;;)
	{
		FString Field;
		if (!FSimpleParse::ParseString(Ptr, Field))
		{
			return false;
		}

		if (!FSimpleParse::MatchZeroOrMoreWhitespace(Ptr) || !FSimpleParse::MatchChar(Ptr, TEXT(':')) || !FSimpleParse::MatchZeroOrMoreWhitespace(Ptr))
		{
			return false;
		}

		if (Field == TEXT("MajorVersion"))
		{
			if (!FSimpleParse::ParseUnsignedNumber(Ptr, OutVersion.MajorVersion))
			{
				return false;
			}

			bParsedMajorVersion = true;
		}
		else if (Field == TEXT("MinorVersion"))
		{
			if (!FSimpleParse::ParseUnsignedNumber(Ptr, OutVersion.MinorVersion))
			{
				return false;
			}

			bParsedMinorVersion = true;
		}
		else if (Field == TEXT("PatchVersion"))
		{
			if (!FSimpleParse::ParseUnsignedNumber(Ptr, OutVersion.PatchVersion))
			{
				return false;
			}

			bParsedPatchVersion = true;
		}
		else if (Field == TEXT("Changelist"))
		{
			if (!FSimpleParse::ParseUnsignedNumber(Ptr, OutVersion.Changelist))
			{
				return false;
			}
		}
		else if (Field == TEXT("CompatibleChangelist"))
		{
			if (!FSimpleParse::ParseUnsignedNumber(Ptr, OutVersion.CompatibleChangelist))
			{
				return false;
			}
		}
		else if (Field == TEXT("IsLicenseeVersion"))
		{
			if (!FSimpleParse::ParseUnsignedNumber(Ptr, OutVersion.IsLicenseeVersion))
			{
				return false;
			}
		}
		else if (Field == TEXT("IsPromotedBuild"))
		{
			if (!FSimpleParse::ParseUnsignedNumber(Ptr, OutVersion.IsPromotedBuild))
			{
				return false;
			}
		}
		else if (Field == TEXT("BranchName"))
		{
			if (!FSimpleParse::ParseString(Ptr, OutVersion.BranchName))
			{
				return false;
			}
		}
		else if (Field == TEXT("BuildId"))
		{
			if (!FSimpleParse::ParseString(Ptr, OutVersion.BuildId))
			{
				return false;
			}
		}
		else if (Field == TEXT("BuildVersion"))
		{
			if (!FSimpleParse::ParseString(Ptr, OutVersion.BuildVersion))
			{
				return false;
			}
		}
		else if (Field == TEXT("BuildURL"))
		{
			if (!FSimpleParse::ParseString(Ptr, OutVersion.BuildUrl))
			{
				return false;
			}
		}
		else
		{
			UE_LOGF(LogCore, Warning, "Unknown field '%ls' found when serializing FBuildVersion", *Field);
			return false;
		}

		if (!FSimpleParse::MatchZeroOrMoreWhitespace(Ptr))
		{
			return false;
		}

		if (FSimpleParse::MatchChar(Ptr, TEXT('}')))
		{
			// Only succeed if we did actually parse these things
			return bParsedMajorVersion && bParsedMinorVersion && bParsedPatchVersion;
		}

		if (!FSimpleParse::MatchChar(Ptr, TEXT(',')) || !FSimpleParse::MatchZeroOrMoreWhitespace(Ptr))
		{
			return false;
		}
	}
}
