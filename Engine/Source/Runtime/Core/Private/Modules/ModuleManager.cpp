// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"

#include "HAL/FileManager.h"
#include "HAL/IConsoleManager.h"
#include "Internationalization/StringTableCore.h"
#include "Logging/StructuredLog.h"
#include "Misc/App.h"
#include "Misc/DataDrivenPlatformInfoRegistry.h"
#include "Misc/DateTime.h"
#include "Misc/FileHelper.h"
#include "Misc/Parse.h"
#include "Misc/Paths.h"
#include "Misc/ScopeExit.h"
#include "Misc/ScopeLock.h"
#include "Modules/ModuleManifest.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "Serialization/LoadTimeTrace.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "Stats/Stats.h"
#include "Trace/Trace.h"
#include "Trace/Trace.inl"
#include "AutoRTFM.h"

DEFINE_LOG_CATEGORY_STATIC(LogModuleManager, Log, All);

#if WITH_HOT_RELOAD
	/** If true, we are reloading a class for HotReload */
	CORE_API bool GIsHotReload = false;
#endif // WITH_HOT_RELOAD

#if UE_MERGED_MODULES

	// Toggle switch for merged modules unloading
	static bool GEnableMergedLibraryUnloading = true;
	static FAutoConsoleVariableRef EnableMergedLibraryUnloadingCVar(
		TEXT("Modules.EnableMergedLibraryUnloading"),
		GEnableMergedLibraryUnloading,
		TEXT("When set, enable the unloading of a merged library when none of its modules are loaded."),
		ECVF_Default);

	// Off switch for unloading specific merged libraries
	static FString GPersistentMergedLibraries;
	static FAutoConsoleVariableRef PersistentMergedLibrariesCVar(
		TEXT("Modules.PersistentMergedLibraries"),
		GPersistentMergedLibraries,
		TEXT("List of comma-separated patterns of merged library names to never unload. The search rule matches any name containing one entry."),
		ECVF_Default);

#if !UE_BUILD_SHIPPING

	// Log user modules of merged libraries
	FAutoConsoleCommand LogMergedLibraryUsageCommand(
		TEXT("Modules.MergedLibraries"),
		TEXT("Log all users of currently loaded merged libraries"),
		FConsoleCommandDelegate::CreateLambda([]()
		{
			FModuleManager::Get().LogMergedLibraryUsage();
		}));

	// Force load an entire merged library
	FAutoConsoleCommand LoadMergedLibraryCommand(
		TEXT("Modules.LoadMergedLibrary"),
		TEXT("Load all modules from a merged library"),
		FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
		{
			if (Args.Num() == 1)
			{
				FModuleManager::Get().LoadAllModulesInMergedLibrary(Args[0]);
			}
		}));

	// Force unload an entire merged library
	FAutoConsoleCommand UnloadMergedLibraryCommand(
		TEXT("Modules.UnloadMergedLibrary"),
		TEXT("Unload all modules from a merged library"),
		FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
		{
			if (Args.Num() == 1)
			{
				FModuleManager::Get().UnloadAllModulesInMergedLibrary(Args[0]);
			}
		}));

#endif // !UE_BUILD_SHIPPING
#endif // UE_MERGED_MODULES

#if WITH_ENGINE
	TMap<UClass*, UClass*>& GetClassesToReinstanceForHotReload()
	{
		static TMap<UClass*, UClass*> Data;
		return Data;
	}
#endif // WITH_ENGINE

FModuleInitializerEntry* GFirstModuleInitializerEntry;

FModuleInitializerEntry::FModuleInitializerEntry(const TCHAR* InName, FInitializeModuleFunctionPtr InFunction, const TCHAR* InName2)
:	Name(InName)
,	Name2(nullptr)
,	Function(InFunction)
{
	if (FCString::Stricmp(InName, InName2) != 0)
	{
		Name2 = InName2;
	}

	Prev = nullptr;
	Next = GFirstModuleInitializerEntry;

	if (GFirstModuleInitializerEntry)
	{
		GFirstModuleInitializerEntry->Prev = this;
	}

	GFirstModuleInitializerEntry = this;
}

FModuleInitializerEntry::~FModuleInitializerEntry()
{
	if (Next)
	{
		Next->Prev = Prev;
	}

	if (Prev)
	{
		Prev->Next = Next;
	}
	else
	{
		GFirstModuleInitializerEntry = Next;
	}
}

FInitializeModuleFunctionPtr FModuleInitializerEntry::FindModule(const TCHAR* Name)
{
	for (FModuleInitializerEntry* Entry = GFirstModuleInitializerEntry; Entry; Entry = Entry->Next)
	{
		if (FCString::Stricmp(Name, Entry->Name) == 0)
		{
			return Entry->Function;
		}
		if (Entry->Name2 && FCString::Stricmp(Name, Entry->Name2) == 0)
		{
			return Entry->Function;
		}
	}
	return nullptr;
}


int32 FModuleManager::FModuleInfo::CurrentLoadOrder = 1;

void FModuleManager::WarnIfItWasntSafeToLoadHere(const FName InModuleName)
{
	if ( !IsInGameThread() )
	{
		UE_LOGF(LogModuleManager, Warning, "ModuleManager: Attempting to load '%ls' outside the main thread.  This module was already loaded - so we didn't crash but this isn't safe.  Please call LoadModule on the main/game thread only.  You can use GetModule or GetModuleChecked instead, those are safe to call outside the game thread.", *InModuleName.ToString());
	}
}

UE_AUTORTFM_ALWAYS_OPEN
FModuleManager::ModuleInfoPtr FModuleManager::FindModule(FName InModuleName)
{
	FModuleManager::ModuleInfoPtr Result = nullptr;

	FScopeLock Lock(&ModulesCriticalSection);
	if ( FModuleManager::ModuleInfoRef* FoundModule = Modules.Find(InModuleName))
	{
		Result = *FoundModule;
	}

	return Result;
}

FModuleManager::ModuleInfoRef FModuleManager::FindModuleChecked(FName InModuleName)
{
	FScopeLock Lock(&ModulesCriticalSection);
	return Modules.FindChecked(InModuleName);
}

// Function level static to allow lazy construction during static initialization
TOptional<FModuleManager>& UE::Core::Private::GetModuleManagerSingleton()
{
	static TOptional<FModuleManager> Singleton(InPlace, FModuleManager::FPrivateToken{});
	return Singleton;
}


void FModuleManager::TearDown()
{
	check(IsInGameThread());
	UE::Core::Private::GetModuleManagerSingleton().Reset();
}

FModuleManager& FModuleManager::Get()
{
	return UE::Core::Private::GetModuleManagerSingleton().GetValue();
}

FModuleManager::FModuleManager(FPrivateToken)
	: bCanProcessNewlyLoadedObjects(false)
	, bExtraBinarySearchPathsAdded(false)
	, bIsLoadingDynamicLibrary(false)
{
	check(IsInGameThread());

#if !IS_MONOLITHIC && !UE_MERGED_MODULES
	// Modules bootstrapping is useful to avoid costly directory enumeration by reloading
	// a serialized state of the module manager. Can only be used when run in the exact
	// same context multiple times (i.e. starting multiple shader compile workers)
	// When using UE_MERGED_MODULES, module initialization is done with statics as if the build was monolithic.
	FString ModulesBootstrapFilename;
	if (FParse::Value(FCommandLine::Get(), TEXT("ModulesBootstrap="), ModulesBootstrapFilename))
	{
		TArray<uint8> FileContent;
		if (FFileHelper::LoadFileToArray(FileContent, *ModulesBootstrapFilename, FILEREAD_Silent))
		{
			FMemoryReader MemoryReader(FileContent, true);
			SerializeStateForBootstrap_Impl(MemoryReader);
		}
		else
		{
			UE_LOGF(LogModuleManager, Display, "Unable to bootstrap from archive %ls, will fallback on normal initialization", *ModulesBootstrapFilename);
		}
	}
#endif
}

FModuleManager::~FModuleManager()
{
	// NOTE: It may not be safe to unload modules by this point (static deinitialization), as other
	//       DLLs may have already been unloaded, which means we can't safely call clean up methods
}

IModuleInterface* FModuleManager::GetModulePtr_Internal(FName ModuleName)
{
	FModuleManager& ModuleManager = FModuleManager::Get();

	ModuleInfoPtr ModuleInfo = ModuleManager.FindModule(ModuleName);
	if (!ModuleInfo.IsValid())
	{
		return nullptr;
	}

	if (!ModuleInfo->Module.IsValid())
	{
		return nullptr;
	}

	return ModuleInfo->Module.Get();
}

void FModuleManager::FindModules(const TCHAR* WildcardWithoutExtension, TArray<FName>& OutModules) const
{
	TArray<FModuleDiskInfo> FoundModules;
	FindModules(WildcardWithoutExtension, FoundModules);
	OutModules.Reserve(OutModules.Num() + FoundModules.Num());
	for (FModuleDiskInfo& Module : FoundModules)
	{
		OutModules.Add(Module.Name);
	}
}

void FModuleManager::FindModules(const TCHAR* WildcardWithoutExtension, TArray<FModuleDiskInfo>& OutModules) const
{
	// @todo plugins: Try to convert existing use cases to use plugins, and get rid of this function

	// Merged modular builds act as if they were monolithic as far as module discovery is concerned.
#if !IS_MONOLITHIC && !UE_MERGED_MODULES

	TMap<FName, FString> ModulePaths;
	FindModulePaths(WildcardWithoutExtension, ModulePaths);

	for(TMap<FName, FString>::TConstIterator Iter(ModulePaths); Iter; ++Iter)
	{
		OutModules.Add(FModuleDiskInfo{ Iter.Key(), Iter.Value() });
	}

#else
	// Check if the wildcard actually contains any wildcard characters. If not, we can do a map lookup instead of iterating.
	bool bContainsWildcardCharacter = false;
	if (WildcardWithoutExtension)
	{
		const TCHAR* WCh = WildcardWithoutExtension;
		while (*WCh)
		{
			if (*WCh == '*' || *WCh == '?')
			{
				bContainsWildcardCharacter = true;
				break;
			}
			WCh++;
		}
	}

	ProcessPendingStaticallyLinkedModuleInitializers();
	if (bContainsWildcardCharacter)
	{
		// There is a wildcard character. Use MatchesWildcard on every key.
		FString Wildcard(WildcardWithoutExtension);
		for (const TPair<FName, FInitializeStaticallyLinkedModule>& It : StaticallyLinkedModuleInitializers)
		{
			if (It.Key.ToString().MatchesWildcard(Wildcard))
			{
				OutModules.Add(FModuleDiskInfo{ It.Key, FString() });
			}
		}
	}
	else
	{
		// There is no wildcard, this could only match one entry matching the name exactly, so do a map lookup instead, which is much faster.
		FName WildcardName(WildcardWithoutExtension);
		if (StaticallyLinkedModuleInitializers.Contains(WildcardName))
		{
			OutModules.Add(FModuleDiskInfo{ WildcardName, FString() });
		}
	}
#endif //  !IS_MONOLITHIC && !UE_MERGED_MODULES
}

bool FModuleManager::ModuleExists(const TCHAR* ModuleName, FString* OutModuleFilePath) const
{
	TArray<FModuleDiskInfo> FoundModules;
	FindModules(ModuleName, FoundModules);
	if (FoundModules.IsEmpty())
	{
		if (OutModuleFilePath)
		{
			OutModuleFilePath->Reset();
		}
		return false;
	}
	else
	{
		if (OutModuleFilePath)
		{
			*OutModuleFilePath = FoundModules[0].FilePath;
		}
		return true;
	}
}

bool FModuleManager::IsModuleLoaded( const FName InModuleName ) const
{
	// Do we even know about this module?
	TSharedPtr<const FModuleInfo, ESPMode::ThreadSafe> ModuleInfoPtr = FindModule(InModuleName);
	if( ModuleInfoPtr.IsValid() )
	{
		const FModuleInfo& ModuleInfo = *ModuleInfoPtr;

		// Only if already loaded
		if( ModuleInfo.Module.IsValid()  )
		{
			// Module is loaded and ready

			// note: not checking (bIsReady || GameThread) , that might be wrong
			//   see difference with GetModule()
			// in fact this function could just be replaced with GetModule() != null
			return true;
		}
	}

	// Not loaded, or not fully initialized yet (StartupModule wasn't called)
	return false;
}

#if !IS_MONOLITHIC
bool FModuleManager::IsModuleUpToDate(const FName InModuleName) const
{
	TMap<FName, FString> ModulePathMap;
	FindModulePaths(*InModuleName.ToString(), ModulePathMap);

	for (const TPair<FName, FString>& Pair : ModulePathMap)
	{
		if (!FPlatformProcess::ModuleExists(Pair.Value))
		{
			return false;
		}
	}

	return ModulePathMap.Num() == 1;
}
#endif

bool FindNewestModuleFile(TArray<FString>& FilesToSearch, const FDateTime& NewerThan, const FString& ModuleFileSearchDirectory, const FString& Prefix, const FString& Suffix, FString& OutFilename)
{
	// Figure out what the newest module file is
	bool bFound = false;
	FDateTime NewestFoundFileTime = NewerThan;

	for (const auto& FoundFile : FilesToSearch)
	{
		// FoundFiles contains file names with no directory information, but we need the full path up
		// to the file, so we'll prefix it back on if we have a path.
		const FString FoundFilePath = ModuleFileSearchDirectory.IsEmpty() ? FoundFile : (ModuleFileSearchDirectory / FoundFile);

		// need to reject some files here that are not numbered...release executables, do have a suffix, so we need to make sure we don't find the debug version
		check(FoundFilePath.Len() > Prefix.Len() + Suffix.Len());
		FString Center = FoundFilePath.Mid(Prefix.Len(), FoundFilePath.Len() - Prefix.Len() - Suffix.Len());
		check(Center.StartsWith(TEXT("-"))); // a minus sign is still considered numeric, so we can leave it.
		if (!Center.IsNumeric())
		{
			// this is a debug DLL or something, it is not a numbered hot DLL
			continue;
		}

		// Check the time stamp for this file
		const FDateTime FoundFileTime = IFileManager::Get().GetTimeStamp(*FoundFilePath);
		if (ensure(FoundFileTime != FDateTime::MinValue()))
		{
			// Was this file modified more recently than our others?
			if (FoundFileTime > NewestFoundFileTime)
			{
				bFound = true;
				NewestFoundFileTime = FoundFileTime;
				OutFilename = FPaths::GetCleanFilename(FoundFilePath);
			}
		}
		else
		{
			// File wasn't found, should never happen as we searched for these files just now
		}
	}

	return bFound;
}

void FModuleManager::AddModuleToModulesList(const FName InModuleName, FModuleManager::ModuleInfoRef& InModuleInfo)
{
	{
		FScopeLock Lock(&ModulesCriticalSection);

		// Update hash table
		Modules.Add(InModuleName, InModuleInfo);
	}

	// List of known modules has changed.  Fire callbacks.
	FModuleManager::Get().ModulesChangedEvent.Broadcast(InModuleName, EModuleChangeReason::PluginDirectoryChanged);
}

void FModuleManager::AddModule(const FName InModuleName)
{
	// Do we already know about this module?  If not, we'll create information for this module now.
	if (!((ensureMsgf(InModuleName != NAME_None, TEXT("FModuleManager::AddModule() was called with an invalid module name (empty string or 'None'.)  This is not allowed.")) &&
		!Modules.Contains(InModuleName))))
	{
		return;
	}

	ModuleInfoRef ModuleInfo(new FModuleInfo());

#if !IS_MONOLITHIC
	RefreshModuleFilenameFromManifestImpl(InModuleName, ModuleInfo.Get());
#endif	// !IS_MONOLITHIC

	// Make sure module info is added to known modules and proper delegates are fired on exit.
	FModuleManager::Get().AddModuleToModulesList(InModuleName, ModuleInfo);
}

#if CPUPROFILERTRACE_ENABLED

UE_TRACE_EVENT_BEGIN(Cpu, LoadModule, NoSync)
UE_TRACE_EVENT_FIELD(UE::Trace::WideString, Name)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Cpu, FPlatformProcess_GetDllHandle, NoSync)
UE_TRACE_EVENT_FIELD(UE::Trace::WideString, Name)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Cpu, StartupModule, NoSync)
UE_TRACE_EVENT_FIELD(UE::Trace::WideString, Name)
UE_TRACE_EVENT_END()

#endif // CPUPROFILERTRACE_ENABLED

#if !IS_MONOLITHIC
void FModuleManager::RefreshModuleFilenameFromManifestImpl(const FName InModuleName, FModuleInfo& ModuleInfo)
{
	FString ModuleNameString = InModuleName.ToString();

	TMap<FName, FString> ModulePathMap;
	FindModulePaths(*ModuleNameString, ModulePathMap);

	if (ModulePathMap.Num() != 1)
	{
		return;
	}

	FString ModuleFilename = MoveTemp(TMap<FName, FString>::TIterator(ModulePathMap).Value());

	const int32 MatchPos = ModuleFilename.Find(ModuleNameString, ESearchCase::IgnoreCase, ESearchDir::FromEnd);

	// If modules are merged it is likely that binary name will not match module. TODO: Revisit this to see if we can get this to work with hot reloading etc
	#if UE_MERGED_MODULES
	if (MatchPos == INDEX_NONE)
	{
		return;
	}
	#endif // UE_MERGED_MODULES

	if (!ensureMsgf(MatchPos != INDEX_NONE, TEXT("Could not find module name '%s' in module filename '%s'"), *InModuleName.ToString(), *ModuleFilename))
	{
		return;
	}

	// Skip any existing module number suffix
	const int32 SuffixStart = MatchPos + ModuleNameString.Len();
	int32 SuffixEnd = SuffixStart;
	if (ModuleFilename[SuffixEnd] == TEXT('-'))
	{
		++SuffixEnd;
		while (FCString::Strchr(TEXT("0123456789"), ModuleFilename[SuffixEnd]))
		{
			++SuffixEnd;
		}

		// Only skip the suffix if it was a number
		if (SuffixEnd - SuffixStart == 1)
		{
			--SuffixEnd;
		}
	}

	const FString Prefix = ModuleFilename.Left(SuffixStart);
	const FString Suffix = ModuleFilename.Right(ModuleFilename.Len() - SuffixEnd);

	// Add this module to the set of modules that we know about
	ModuleInfo.OriginalFilename = Prefix + Suffix;
	ModuleInfo.Filename         = ModuleFilename;
}

void FModuleManager::RefreshModuleFilenameFromManifest(const FName InModuleName)
{
	if (ModuleInfoPtr ModuleInfoPtr = FindModule(InModuleName))
	{
		this->RefreshModuleFilenameFromManifestImpl(InModuleName, *ModuleInfoPtr);
	}
}

void* FModuleManager::InternalLoadLibrary(FName ModuleName, const FString& LibraryToLoad)
{
	UE_LOGF(LogModuleManager, Log, "InternalLoadLibrary: '%ls' ('%ls')", *ModuleName.ToString(), *LibraryToLoad);

	LLM_SCOPE_BYNAME(TEXT("Modules"));

	void* Handle = nullptr;

#if CPUPROFILERTRACE_ENABLED
	UE_TRACE_LOG_SCOPED_T(Cpu, FPlatformProcess_GetDllHandle, CpuChannel)
		<< FPlatformProcess_GetDllHandle.Name(*LibraryToLoad);
#endif // CPUPROFILERTRACE_ENABLED

#if UE_MERGED_MODULES

	// Stop any delay unload for that library, and match the handle for the code module to the library path.
	DelayUnloadLibraries.Remove(LibraryToLoad);
	if (void** ExistingHandle = LibraryHandles.Find(LibraryToLoad))
	{
		UE_LOGF(LogModuleManager, Log, "InternalLoadLibrary: already loaded library '%ls'", *LibraryToLoad);

		return ExistingHandle;
	}

	// The plugin manager normally handles dependencies, but merged libraries introduce indirect dependencies.
	// If we load plugin A that's merged with B, and B depends on C in another library...
	// ... then we need to load C first, but the plugin manager cannot know that.
	TArray<FName> Dependencies = GetAllSharedLibrariesForLibrary(LibraryToLoad);
	for (FName Dependency : Dependencies)
	{
		if (LibraryHandles.Find(Dependency.ToString()) == nullptr)
		{
			UE_LOGF(LogModuleManager, Log, "InternalLoadLibrary: loading dependency '%ls'", *Dependency.ToString());
			InternalLoadLibrary(NAME_None, Dependency.ToString());
		}
	}

#endif // UE_MERGED_MODULES

	// Do the actual loading
	const uint64_t InitialMemUsed = FPlatformMemory::GetMemoryUsedFast();
	{
		TGuardValue<bool> IsLoadingDynamicLibraryGuard(bIsLoadingDynamicLibrary, true);
		Handle = FPlatformProcess::GetDllHandle(*LibraryToLoad);
	}
	const uint64_t FinalMemUsed = FPlatformMemory::GetMemoryUsedFast();

#if UE_MERGED_MODULES

	// Stop any delay unload for that library, and match the handle for the code module to the library path.
	LibraryHandles.Add(LibraryToLoad, Handle);

#endif // UE_MERGED_MODULES

	UE_LOGF(LogModuleManager, Verbose, "InternalLoadLibrary: used about %llu KB", (FinalMemUsed - InitialMemUsed) / 1024);

	return Handle;
}

void FModuleManager::InternalFreeLibrary(FName ModuleName, void* Handle)
{
	UE_LOGF(LogModuleManager, Verbose, "InternalFreeLibrary: '%ls'", *ModuleName.ToString());

#if UE_MERGED_MODULES

	// When using merged modular build, unloading follows a reference-counting and delay-unload process.
	// We count how many modules are referencing each merged dynamic library, and decrement that value when one is freed.
	// When no module uses the library anymore, the following process will happen:
	//     - Mark the library for delay unload
	//     - Notify the UObject system to remove live classes in every module of that library
	//     - Wait for garbage collection to be run
	//     - In FModuleManager::OnObjectCleanup, after GC, actually do the removal using FreeDllHandle
	//     - Afterwards, unused shared libraries will call InternalFreeLibrary

	int32 ModuleCount = 0;
	FString LibraryToUnload;

	// Explicit module: get the path and handle from there
	if (ModuleName != NAME_None)
	{
		ModuleInfoRef ModuleInfo = Modules[ModuleName];
		LibraryToUnload = FPaths::ConvertRelativePathToFull(ModuleInfo->Filename);
		ModuleCount = GetLoadedModulesForLibrary(LibraryToUnload).Num();
		UE_LOGF(LogModuleManager, Verbose, "InternalFreeLibrary: library '%ls' has %d users", *LibraryToUnload, ModuleCount);
	}

	// Full library unload, when a shared library goes unused
	else if (Handle)
	{
		for (const TPair<FString, void*>& LibraryNameAndHandle : LibraryHandles)
		{
			if (LibraryNameAndHandle.Value == Handle)
			{
				LibraryToUnload = LibraryNameAndHandle.Key;
				UE_LOGF(LogModuleManager, Verbose, "InternalFreeLibrary: library '%ls' was explicitly requested", *LibraryToUnload);
				break;
			}
		}
	}
	else
	{
		UE_LOGF(LogModuleManager, Fatal, "InternalFreeLibrary: no module name or valid handle was passed")
	}

	// Safety feature: allow disabling of merged library unloading, either wholly, or for specific libraries
	const auto CanUnloadLibrary = [&LibraryToUnload, this]()
	{
		if (!GEnableMergedLibraryUnloading)
		{
			return false;
		}

		if (DelayUnloadLibraries.Contains(LibraryToUnload))
		{
			return false;
		}

		if (GPersistentMergedLibraries.Len())
		{
			TArray<FString> PersistentLibraries;
			GPersistentMergedLibraries.ParseIntoArray(PersistentLibraries, TEXT(","));

			for (const FString& PersistentLibrary : PersistentLibraries)
			{
				if (LibraryToUnload.Contains(PersistentLibrary))
				{
					return false;
				}
			}
		}

		return true;
	};

	if (ModuleCount == 0 && CanUnloadLibrary())
	{
		UE_LOGF(LogModuleManager, Log, "InternalFreeLibrary: preparing unload for library '%ls'", *LibraryToUnload);

		TArray<FName> UnloadedModules;

		// Identify modules to unload by their statically linked identifier & their library path,
		// then remove statically linked initializer info - if it exists it means the module needs cleanup
		for (TPair<FName, FInitializeStaticallyLinkedModule> OtherModuleNameAndInfo : StaticallyLinkedModuleInitializers)
		{
			TMap<FName, FString> OtherModulesPathMap;
			FindModulePaths(*OtherModuleNameAndInfo.Key.ToString(), OtherModulesPathMap);
			const FString* OtherModuleLibraryNamePtr = OtherModulesPathMap.Find(OtherModuleNameAndInfo.Key);

			if (OtherModuleLibraryNamePtr && *OtherModuleLibraryNamePtr == LibraryToUnload)
			{
				UE_LOGF(LogModuleManager, Verbose, "InternalFreeLibrary: cleaning up for module '%ls'",
					*OtherModuleNameAndInfo.Key.ToString());

				UnloadedModules.Add(OtherModuleNameAndInfo.Key);
			}
		}

		for (FName UnloadedModule : UnloadedModules)
		{
			StaticallyLinkedModuleInitializers.Remove(UnloadedModule);
		}

		// Mark for delay unload to run after GC is done.
		DelayUnloadLibraries.Add(LibraryToUnload, UnloadedModules);

		// If the module to unload has UObjects in it, then the UObject system need be informed.
		// UObjects should be removed and garbage collection should run.
		OnModulesUnloadCallback.Broadcast(UnloadedModules);
	}

	// Some merged libraries are shared, meaning they are potentially depended on by multiple merged libraries; unload any that's unused.
	// This is NOT done when ModuleName isn't set to avoid recursion; only non-shared, merged, dynamic libraries can have dependencies.
	if (ModuleName != NAME_None)
	{
		for (const TPair<FString, void*>& LibraryNameAndHandle : LibraryHandles)
		{
			const FString LibraryName = FPaths::ConvertRelativePathToFull(LibraryNameAndHandle.Key);
			const TArray<FName> LoadedModules = GetLoadedModulesForLibrary(LibraryName);
			const TArray<FName> LoadedLibraries = GetLoadedLibrariesForSharedLibrary(LibraryName);

			if (LoadedModules.IsEmpty() && LoadedLibraries.IsEmpty())
			{
				UE_LOGF(LogModuleManager, Log, "InternalFreeLibrary: '%ls' is now unused", *LibraryName);

				InternalFreeLibrary(NAME_None, LibraryNameAndHandle.Value);
			}
		}
	}

#else
	FPlatformProcess::FreeDllHandle(Handle);
#endif // UE_MERGED_MODULES
}

#if UE_MERGED_MODULES

TArray<FName> FModuleManager::GetLoadedModulesForLibrary(const FString& LibraryFilename) const
{
	TArray<FName> UsingModules;

	for (const TPair<FName, ModuleInfoRef>& ModuleNameAndInfo : Modules)
	{
		const ModuleInfoRef& ModuleInfo = ModuleNameAndInfo.Value;
		if (ModuleInfo->Module.IsValid() && !ModuleInfo->Filename.IsEmpty() && LibraryFilename == FPaths::ConvertRelativePathToFull(ModuleInfo->Filename))
		{
			UsingModules.Add(ModuleNameAndInfo.Key);
		}
	}

	return UsingModules;
}

TArray<FName> FModuleManager::GetLoadedLibrariesForSharedLibrary(const FString& LibraryFilename) const
{
	TArray<FName> UsingLibraries;

	// Walk over all dependency entries to find any loaded library directly depending on this one.
	for (const TPair<FString, TArray<FString>>& LibraryAndDependenciesPair : LibraryDependencies)
	{
		// Find out if there's an active handle for this library.
		// LibraryHandles use full filenames, but LibraryDependencies only has the filename, so we can't just Find() it.
		FString OtherLibraryFilename;
		for (const TPair<FString, void*>& LibraryHandlePair : LibraryHandles)
		{
			if (LibraryHandlePair.Key.EndsWith(LibraryAndDependenciesPair.Key))
			{
				OtherLibraryFilename = LibraryHandlePair.Key;
				break;
			}
		}

		// Find out if this other active library depends on the library we're looking for.
		if (!OtherLibraryFilename.IsEmpty())
		{
			for (const FString& Dependency : LibraryAndDependenciesPair.Value)
			{
				if (LibraryFilename.EndsWith(Dependency))
				{
					UsingLibraries.Add(*OtherLibraryFilename);
					break;
				}
			}
		}
	}

	return UsingLibraries;
}

TArray<FName> FModuleManager::GetAllSharedLibrariesForLibrary(const FString& LibraryFilename) const
{
	TArray<FName> AllSharedLibraries;

	for (const TPair<FString, TArray<FString>>& LibraryAndDependenciesPair : LibraryDependencies)
	{
		if (LibraryFilename.EndsWith(LibraryAndDependenciesPair.Key))
		{
			for (const FString& Dependency : LibraryAndDependenciesPair.Value)
			{
				const FString DependencyFilename = FPaths::Combine(FPlatformProcess::GetModulesDirectory(), Dependency);
				AllSharedLibraries.AddUnique(*FPaths::ConvertRelativePathToFull(DependencyFilename));
			}

			break;
		}
	}

	return AllSharedLibraries;
}

#endif // UE_MERGED_MODULES

#if UE_MERGED_MODULES && !UE_BUILD_SHIPPING

void FModuleManager::LogMergedLibraryUsage()
{
	for (const TPair<FString, void*>& LibraryNameAndHandle : LibraryHandles)
	{
		const FString LibraryName = FPaths::ConvertRelativePathToFull(LibraryNameAndHandle.Key);
		const TArray<FName> LoadedModules = GetLoadedModulesForLibrary(LibraryName);
		const TArray<FName> SharedLibraries = GetAllSharedLibrariesForLibrary(LibraryName);
		const TArray<FName> LoadedLibraries = GetLoadedLibrariesForSharedLibrary(LibraryName);

		UE_LOGF(LogModuleManager, Log, "Merged library usage for '%ls'", *LibraryName);

		if (LoadedModules.Num())
		{
			UE_LOGF(LogModuleManager, Log, "  %d modules loaded for this library", LoadedModules.Num());
			for (const FName& Module : LoadedModules)
			{
				UE_LOGF(LogModuleManager, Log, "    '%ls'", *Module.ToString());
			}
		}

		if (SharedLibraries.Num())
		{
			UE_LOGF(LogModuleManager, Log, "  depends on %d other libraries", SharedLibraries.Num());
			for (const FName& Library : SharedLibraries)
			{
				UE_LOGF(LogModuleManager, Log, "    '%ls'", *Library.ToString());
			}
		}

		if (LoadedLibraries.Num())
		{
			UE_LOGF(LogModuleManager, Log, "  %d other libraries depend on this library", LoadedLibraries.Num());
			for (const FName& Library : LoadedLibraries)
			{
				UE_LOGF(LogModuleManager, Log, "    '%ls'", *Library.ToString());
			}
		}
	}
}

void FModuleManager::LoadAllModulesInMergedLibrary(FStringView LibraryName)
{
	for (const TPair<FName, ModuleInfoRef>& ModuleNameAndInfo : Modules)
	{
		const ModuleInfoRef& ModuleInfo = ModuleNameAndInfo.Value;
		if (ModuleInfo->Filename.Contains(LibraryName))
		{
			LoadModule(ModuleNameAndInfo.Key);
		}
	}
}

void FModuleManager::UnloadAllModulesInMergedLibrary(FStringView LibraryName)
{
	for (const TPair<FName, ModuleInfoRef>& ModuleNameAndInfo : Modules)
	{
		const ModuleInfoRef& ModuleInfo = ModuleNameAndInfo.Value;
		if (ModuleInfo->Module.IsValid() && ModuleInfo->Filename.Contains(LibraryName))
		{
			UnloadModule(ModuleNameAndInfo.Key, false, true);
		}
	}
}

#endif // UE_MERGED_MODULES && !UE_BUILD_SHIPPING

#endif	// !IS_MONOLITHIC

void FModuleManager::OnObjectCleanup()
{
#if UE_MERGED_MODULES

	// See FModuleManager::InternalFreeLibrary for a detailed rundown of the library unload process.
	for (auto LibraryAndModulesIterator = DelayUnloadLibraries.CreateIterator(); LibraryAndModulesIterator; ++LibraryAndModulesIterator)
	{
		// Check that no live object still exists for the modules in this merged library
		if (ensure(CheckLiveObjectsInModulesCallback.IsBound()) && !CheckLiveObjectsInModulesCallback.Execute(LibraryAndModulesIterator.Value()))
		{
			void** HandlePtr = LibraryHandles.Find(LibraryAndModulesIterator.Key());
			if (HandlePtr)
			{
				UE_LOGF(LogModuleManager, Log, "OnObjectCleanup: unloading dynamic library '%ls'", *LibraryAndModulesIterator.Key());

				AutoRTFM::UnreachableIfClosed();

				// Give various systems a last chance to clean up before removal
				OnLibraryUnmountCallback.Broadcast();

#if UE_DELEGATE_CHECK_LIFETIME
				FMulticastDelegateTracker::CheckForStaleDelegatesInModules(LibraryAndModulesIterator.Value());
#endif // UE_DELEGATE_CHECK_LIFETIME

				const uint64_t InitialMemUsed = FPlatformMemory::GetMemoryUsedFast();
				FPlatformProcess::FreeDllHandle(*HandlePtr);
				const uint64_t FinalMemUsed = FPlatformMemory::GetMemoryUsedFast();

				UE_LOGF(LogModuleManager, Verbose, "OnObjectCleanup: freed about %llu KB", (InitialMemUsed - FinalMemUsed) / 1024);
				LibraryHandles.Remove(LibraryAndModulesIterator.Key());
			}

			LibraryAndModulesIterator.RemoveCurrent();
		}
	}

#endif // UE_MERGED_MODULES
}

IModuleInterface* FModuleManager::LoadModule(const FName InModuleName, ELoadModuleFlags InLoadModuleFlags)
{
	EModuleLoadResult FailureReason = EModuleLoadResult::Success;
	return GetOrLoadModule(InModuleName, FailureReason, InLoadModuleFlags);
}

IModuleInterface* FModuleManager::GetOrLoadModule(const FName InModuleName, EModuleLoadResult& OutFailureReason, ELoadModuleFlags InLoadModuleFlags)
{
	LLM_SCOPE_BYNAME(TEXT("Modules"));
	// We allow an already loaded module to be returned in other threads to simplify
	// parallel processing scenarios but they must have been loaded from the main thread beforehand.
	IModuleInterface* Module = GetModule(InModuleName);
	if (Module)
	{
		return Module;
	}
	else if (!IsInGameThread())
	{
		OutFailureReason = EModuleLoadResult::NotLoadedByGameThread;
		return Module;
	}

	IModuleInterface* Result = LoadModuleWithFailureReason(InModuleName, OutFailureReason, InLoadModuleFlags);

	// This should return a valid pointer only if and only if the module is loaded
	checkSlow((Result != nullptr) == IsModuleLoaded(InModuleName));

	return Result;
}

const TCHAR* LexToString(EModuleLoadResult LoadResult)
{
	switch (LoadResult)
	{
	case EModuleLoadResult::Success:				return TEXT("Success");
	case EModuleLoadResult::FileNotFound:			return TEXT("FileNotFound");
	case EModuleLoadResult::FileIncompatible:		return TEXT("FileIncompatible");
	case EModuleLoadResult::CouldNotBeLoadedByOS:	return TEXT("CouldNotBeLoadedByOS");
	case EModuleLoadResult::FailedToInitialize:		return TEXT("FailedToInitialize");
	case EModuleLoadResult::NotLoadedByGameThread:	return TEXT("NotLoadedByGameThread");
	default:										return TEXT("<Unknown>");
	}
}

IModuleInterface& FModuleManager::LoadModuleChecked( const FName InModuleName )
{
	EModuleLoadResult FailureReason = EModuleLoadResult::Success;
	IModuleInterface* Module = GetOrLoadModule(InModuleName, FailureReason, ELoadModuleFlags::LogFailures);

	checkf(Module, TEXT("ModuleName=%s, Failure=%s, IsInGameThread=%s"),
		*InModuleName.ToString(),
		LexToString(FailureReason),
		IsInGameThread() ? TEXT("Yes") : TEXT("No"));

	return *Module;
}

IModuleInterface* FModuleManager::LoadModuleWithFailureReason(const FName InModuleName, EModuleLoadResult& OutFailureReason, ELoadModuleFlags InLoadModuleFlags)
{
	IModuleInterface* LoadedModule = nullptr;
	OutFailureReason = EModuleLoadResult::Success;
	
	// note that this behaves differently than ::LoadModule(), when called from not-game-thread
	//	 LoadModule just redirects to ::GetModule on non-game-thread
	
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	WarnIfItWasntSafeToLoadHere(InModuleName);
#endif

	// Do fast check for existing module, this is the most common case
	ModuleInfoPtr FoundModulePtr = FindModule(InModuleName);

	const auto EnsureModuleFilename = [this, InModuleName, &FoundModulePtr]()
	{
#if UE_MERGED_MODULES
		ModuleInfoRef ModuleInfo = FoundModulePtr.ToSharedRef();
		if (ModuleInfo->Filename.IsEmpty() || !FPlatformProcess::ModuleExists(ModuleInfo->Filename))
		{
			TMap<FName, FString> ModulePathMap;
			FindModulePaths(*InModuleName.ToString(), ModulePathMap);
			if (ModulePathMap.Num() == 1)
			{
				ModuleInfo->Filename = MoveTemp(TMap<FName, FString>::TIterator(ModulePathMap).Value());
			}
		}
#endif // UE_MERGED_MODULES
	};

	if (FoundModulePtr.IsValid())
	{
		LoadedModule = FoundModulePtr->Module.Get();

		if (LoadedModule)
		{
			EnsureModuleFilename();

			// note: this function does not check (bIsReady || IsInGameThread()) the way GetModule() does
			//   that looks like a bug if called from off-game-thread

			return LoadedModule;
		}
	}

	UE_LOGF(LogModuleManager, Verbose, "LoadModuleWithFailureReason %ls", *InModuleName.ToString());

	// doing LoadModule off GameThread should not be done
	// already warned above
	// enable this ensure when we know it's okay
//	ensureMsgf(IsInGameThread(), TEXT("ModuleManager: Attempting to load '%s' outside the main thread.  Please call LoadModule on the main/game thread only.  You can use GetModule or GetModuleChecked instead, those are safe to call outside the game thread."), *InModuleName.ToString());

	UE_SCOPED_ENGINE_ACTIVITY(TEXT("Loading Module %s"), *InModuleName.ToString());
	FScopedBootTiming BootTimingScope("LoadModule");
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("Module Load"), STAT_ModuleLoad, STATGROUP_LoadTime);
#if CPUPROFILERTRACE_ENABLED
	UE_TRACE_LOG_SCOPED_T(Cpu, LoadModule, CpuChannel)
		<< LoadModule.Name(*InModuleName.ToString());
#endif // CPUPROFILERTRACE_ENABLED

#if	STATS
	// This is fine here, we only load a handful of modules.
	static FString Module = "Module";
	const FString LongName = Module / InModuleName.GetPlainNameString();
	const TStatId StatId = FDynamicStats::CreateStatId<FStatGroup_STATGROUP_UObjects>( LongName );
	FScopeCycleCounter CycleCounter( StatId );
#endif // STATS

	if (!FoundModulePtr.IsValid())
	{
		// Update our set of known modules, in case we don't already know about this module
		AddModule(InModuleName);

		// Ptr will always be valid at this point
		FoundModulePtr = FindModule(InModuleName);

		// NOTE: Module is now findable , calls to Find or Load Module will find this module pointer
		//  but it's not initialized yet (bIsReady is false so Get from other threads will fail)
		// this AddModule must be done before the module is initialized
		// because StartupModule may call functions that Find/Load on this module
		// and they should get back this pointer, even though it is not finished initializing yet
	}
	
	// Grab the module info.  This has the file name of the module, as well as other info.
	ModuleInfoRef ModuleInfo = FoundModulePtr.ToSharedRef();

	// Make sure this isn't a module that we had previously loaded, and then unloaded at shutdown time.
	//
	// If this assert goes off, your trying to load a module during the shutdown phase that was already
	// cleaned up.  The easiest way to fix this is to change your code to query for an already-loaded
	// module instead of trying to load it directly.
	checkf((!ModuleInfo->bWasUnloadedAtShutdown), TEXT("Attempted to load module '%s' that was already unloaded at shutdown.  FModuleManager::LoadModule() was called to load a module that was previously loaded, and was unloaded at shutdown time.  If this assert goes off, you're trying to load a module during the shutdown phase that was already cleaned up.  The easiest way to fix this is to change your code to query for an already-loaded module instead of trying to load it directly."), *InModuleName.ToString());

	// Initialize a module that was registered with the module manager using the static initializer.
	// In monolithic builds, we should know about this module before we get to load it; using merged modules this can be delayed until the library has been loaded.
	const auto InitializeModule = [this, InModuleName, InLoadModuleFlags, &ModuleInfo, &LoadedModule, &OutFailureReason, &EnsureModuleFilename](const FInitializeStaticallyLinkedModule* ModuleInitializerPtr)
	{
		const FInitializeStaticallyLinkedModule& ModuleInitializer(*ModuleInitializerPtr);

		// Initialize the module!
		ModuleInfo->Module = TUniquePtr<IModuleInterface>(ModuleInitializer.Execute());

		if (ModuleInfo->Module.IsValid())
		{
			FScopedBootTiming BootScope("LoadModule  - ", InModuleName);
			TRACE_LOADTIME_REQUEST_GROUP_SCOPE(TEXT("LoadModule - %s"), *InModuleName.ToString());

			{
				// Defer String Table find/load during CDO registration, as it may happen 
				// before StartupModule has had a chance to load the String Table
				IStringTableEngineBridge::FScopedDeferFindOrLoad DeferStringTableFindOrLoad;

				ProcessLoadedObjectsCallback.Broadcast(InModuleName, bCanProcessNewlyLoadedObjects);
			}

			// Startup the module
			{
#if CPUPROFILERTRACE_ENABLED
				UE_TRACE_LOG_SCOPED_T(Cpu, StartupModule, CpuChannel)
					<< StartupModule.Name(*InModuleName.ToString());
#endif // CPUPROFILERTRACE_ENABLED
				ModuleInfo->Module->StartupModule();
			}

			// The module might try to load other dependent modules in StartupModule. In this case, we want those modules shut down AFTER this one because we may still depend on the module at shutdown.
			ModuleInfo->LoadOrder = FModuleInfo::CurrentLoadOrder++;

			// It's now ok for other threads to use the module.
			ModuleInfo->bIsReady = true;

			// Module was started successfully!  Fire callbacks.
			ModulesChangedEvent.Broadcast(InModuleName, EModuleChangeReason::ModuleLoaded);

			// Set the return parameter
			EnsureModuleFilename();
			LoadedModule = ModuleInfo->Module.Get();
		}
		else
		{
			UE_CLOGF((InLoadModuleFlags & ELoadModuleFlags::LogFailures) != ELoadModuleFlags::None,
				LogModuleManager, Warning, "ModuleManager: Unable to load module '%ls' because InitializeModule function failed (returned nullptr.)", *InModuleName.ToString());

			OutFailureReason = EModuleLoadResult::FailedToInitialize;
		}
	};

	ProcessPendingStaticallyLinkedModuleInitializers();
	const FInitializeStaticallyLinkedModule* ModuleInitializerPtr = StaticallyLinkedModuleInitializers.Find(InModuleName);
	if (ModuleInitializerPtr != nullptr)
	{
		InitializeModule(ModuleInitializerPtr);
	}
#if IS_MONOLITHIC
	else
	{
		// Monolithic builds that do not have the initializer were *not found* during the build step, so return FileNotFound
		// (FileNotFound is an acceptable error in some case - ie loading a content only project)
		UE_CLOGF((InLoadModuleFlags & ELoadModuleFlags::LogFailures) != ELoadModuleFlags::None,
			LogModuleManager, Warning, "ModuleManager: Module '%ls' not found - its StaticallyLinkedModuleInitializers function is null.", *InModuleName.ToString());

		OutFailureReason = EModuleLoadResult::FileNotFound;
	}
#else
	else
	{
		// Make sure that any UObjects that need to be registered were already processed before we go and
		// load another module.  We just do this so that we can easily tell whether UObjects are present
		// in the module being loaded.
		if (bCanProcessNewlyLoadedObjects)
		{
			// Defer String Table find/load during CDO registration, as it may happen 
			// before StartupModule has had a chance to load the String Table
			IStringTableEngineBridge::FScopedDeferFindOrLoad DeferStringTableFindOrLoad;

			ProcessLoadedObjectsCallback.Broadcast(NAME_None, bCanProcessNewlyLoadedObjects);
		}

		// Try to dynamically load the DLL

		UE_LOGF(LogModuleManager, Verbose, "ModuleManager: Load Module '%ls' DLL '%ls'", *InModuleName.ToString(), *ModuleInfo->Filename);

		if (ModuleInfo->Filename.IsEmpty() || !FPlatformProcess::ModuleExists(ModuleInfo->Filename))
		{
			TMap<FName, FString> ModulePathMap;
			FindModulePaths(*InModuleName.ToString(), ModulePathMap);

			if (ModulePathMap.Num() != 1)
			{
				UE_CLOGF((InLoadModuleFlags & ELoadModuleFlags::LogFailures) != ELoadModuleFlags::None,
					LogModuleManager, Warning, "ModuleManager: Unable to load module '%ls'  - %d instances of that module name found.", *InModuleName.ToString(), ModulePathMap.Num());

				OutFailureReason = EModuleLoadResult::FileNotFound;
				return nullptr;
			}

			ModuleInfo->Filename = MoveTemp(TMap<FName, FString>::TIterator(ModulePathMap).Value());
		}

		// Determine which file to load for this module.
		const FString ModuleFileToLoad = FPaths::ConvertRelativePathToFull(ModuleInfo->Filename);

		// Clear the handle and set it again below if the module is successfully loaded
		ModuleInfo->Handle = nullptr;

		// Skip this check if file manager has not yet been initialized
		if (FPlatformProcess::ModuleExists(ModuleFileToLoad))
		{
			ModuleInfo->Handle = InternalLoadLibrary(InModuleName, ModuleFileToLoad);
			
			if (ModuleInfo->Handle != nullptr)
			{
#if UE_MERGED_MODULES

				// We just loaded a library and so we have a lot of pending initializers ready to go, including the one for our module
				ProcessPendingStaticallyLinkedModuleInitializers();
				const FInitializeStaticallyLinkedModule* MergedModuleInitializerPtr = StaticallyLinkedModuleInitializers.Find(InModuleName);
				if (MergedModuleInitializerPtr != nullptr)
				{
					InitializeModule(MergedModuleInitializerPtr);
				}

#endif // UE_MERGED_MODULES

				{
					// Defer String Table find/load during CDO registration, as it may happen 
					// before StartupModule has had a chance to load the String Table
					IStringTableEngineBridge::FScopedDeferFindOrLoad DeferStringTableFindOrLoad;

					// First things first.  If the loaded DLL has UObjects in it, then their generated code's
					// static initialization will have run during the DLL loading phase, and we'll need to
					// go in and make sure those new UObject classes are properly registered.
					// Sometimes modules are loaded before even the UObject systems are ready.  We need to assume
					// these modules aren't using UObjects.
					// OK, we've verified that loading the module caused new UObject classes to be
					// registered, so we'll treat this module as a module with UObjects in it.
					ProcessLoadedObjectsCallback.Broadcast(InModuleName, bCanProcessNewlyLoadedObjects);
				}


				// Find our "Initialize<Name>Module" global function, which must exist for all module DLLs
				FInitializeModuleFunctionPtr InitializeModuleFunctionPtr = FModuleInitializerEntry::FindModule(*InModuleName.ToString());

#if !UE_MERGED_MODULES
				if (!InitializeModuleFunctionPtr)
				{
					// If not found this might be some special case module so look for "InitializeModule" global function
					InitializeModuleFunctionPtr = (FInitializeModuleFunctionPtr)FPlatformProcess::GetDllExport(ModuleInfo->Handle, TEXT("InitializeModule"));
				}
#endif //  !UE_MERGED_MODULES

				if (InitializeModuleFunctionPtr != nullptr)
				{
					if ( ModuleInfo->Module.IsValid() )
					{
						// Assign the already loaded module into the return value, otherwise the return value gives the impression the module failed load!
						LoadedModule = ModuleInfo->Module.Get();
					}
					else
					{
						// Initialize the module!
						ModuleInfo->Module = TUniquePtr<IModuleInterface>(InitializeModuleFunctionPtr());

						if (ModuleInfo->Module.IsValid())
						{
							// Startup the module
							{
#if CPUPROFILERTRACE_ENABLED
								UE_TRACE_LOG_SCOPED_T(Cpu, StartupModule, CpuChannel)
									<< StartupModule.Name(*InModuleName.ToString());
#endif // CPUPROFILERTRACE_ENABLED
								ModuleInfo->Module->StartupModule();
							}

							// The module might try to load other dependent modules in StartupModule. In this case, we want those modules shut down AFTER this one because we may still depend on the module at shutdown.
							ModuleInfo->LoadOrder = FModuleInfo::CurrentLoadOrder++;

							// It's now ok for other threads to use the module.
							ModuleInfo->bIsReady = true;

							// Module was started successfully!  Fire callbacks.
							ModulesChangedEvent.Broadcast(InModuleName, EModuleChangeReason::ModuleLoaded);

							// Set the return parameter
							EnsureModuleFilename();
							LoadedModule = ModuleInfo->Module.Get();
						}
						else
						{
							UE_CLOGF((InLoadModuleFlags & ELoadModuleFlags::LogFailures) != ELoadModuleFlags::None,
								LogModuleManager, Warning, "ModuleManager: Unable to load module '%ls' because InitializeModule function failed (returned nullptr.)", *ModuleFileToLoad);

							InternalFreeLibrary(InModuleName, ModuleInfo->Handle);
							ModuleInfo->Handle = nullptr;
							OutFailureReason = EModuleLoadResult::FailedToInitialize;
						}
					}
				}

#if !UE_MERGED_MODULES

				// This is normal with merged modules, as we don't have a single module for each library
				else
				{
					UE_CLOGF((InLoadModuleFlags & ELoadModuleFlags::LogFailures) != ELoadModuleFlags::None,
						LogModuleManager, Warning, "ModuleManager: Unable to load module '%ls' because InitializeModule function was not found.", *ModuleFileToLoad);

					InternalFreeLibrary(InModuleName, ModuleInfo->Handle);
					ModuleInfo->Handle = nullptr;
					OutFailureReason = EModuleLoadResult::FailedToInitialize;
				}

#endif // !UE_MERGED_MODULES

			}
			else
			{
				UE_CLOGF((InLoadModuleFlags & ELoadModuleFlags::LogFailures) != ELoadModuleFlags::None,
					LogModuleManager, Warning, "ModuleManager: Unable to load module '%ls' because the file couldn't be loaded by the OS.", *ModuleFileToLoad);

				OutFailureReason = EModuleLoadResult::CouldNotBeLoadedByOS;
			}
		}
		else
		{
			UE_CLOGF((InLoadModuleFlags & ELoadModuleFlags::LogFailures) != ELoadModuleFlags::None,
				LogModuleManager, Warning, "ModuleManager: Unable to load module '%ls' because the file '%ls' was not found.", *InModuleName.ToString(), *ModuleFileToLoad);

			OutFailureReason = EModuleLoadResult::FileNotFound;
		}
	}
#endif

	return LoadedModule;
}

bool FModuleManager::UnloadModule( const FName InModuleName, bool bIsShutdown, bool bAllowUnloadCode)
{
	UE_LOGF(LogModuleManager, Verbose, "UnloadModule %ls %d", *InModuleName.ToString(), bAllowUnloadCode);

	// Do we even know about this module?
	ModuleInfoPtr ModuleInfoPtr = FindModule(InModuleName);
	if( ModuleInfoPtr.IsValid() )
	{
		FModuleInfo& ModuleInfo = *ModuleInfoPtr;

		// Only if already loaded
		if( ModuleInfo.Module.IsValid() )
		{
			// If we are running in a transaction we defer the unload until we know the body of
			// the transaction is definitely going to be committed.
			if (AutoRTFM::IsClosed())
			{
				AutoRTFM::OnCommit([this, InModuleName, bIsShutdown, bAllowUnloadCode]
				{
					this->UnloadModule(InModuleName, bIsShutdown, bAllowUnloadCode);
				});
				return true;
			}

			// Will offer use-before-ready protection at next reload
			ModuleInfo.bIsReady = false;

			// Shutdown the module
			ModuleInfo.Module->ShutdownModule();

			// Release reference to module interface.  This will actually destroy the module object.
			ModuleInfo.Module.Reset();

#if IS_MONOLITHIC

#if UE_DELEGATE_CHECK_LIFETIME
			FMulticastDelegateTracker::CheckForStaleDelegatesInModules({ InModuleName });
#endif // UE_DELEGATE_CHECK_LIFETIME

#else

#if UE_MERGED_MODULES
			// Double-check that we don't have a valid handle for this...
			if (ModuleInfo.Handle == nullptr)
			{
				void** HandlePtr = LibraryHandles.Find(ModuleInfo.Filename);
				if (HandlePtr)
				{
					ModuleInfo.Handle = *HandlePtr;
				}
			}
#endif // UE_MERGED_MODULES

			if( ModuleInfo.Handle != nullptr )
			{
				// If we're shutting down then don't bother actually unloading the DLL.  We'll simply abandon it in memory
				// instead.  This makes it much less likely that code will be unloaded that could still be called by
				// another module, such as a destructor or other virtual function.  The module will still be unloaded by
				// the operating system when the process exits.
				if( !bIsShutdown && bAllowUnloadCode )
				{
					// Unload the DLL
					InternalFreeLibrary( InModuleName, ModuleInfo.Handle );
				}
				ModuleInfo.Handle = nullptr;
			}
#endif // IS_MONOLITHIC

			// If we're shutting down, then we never want this module to be "resurrected" in this session.
			// It's gone for good!  So we'll mark it as such so that we can catch cases where a routine is
			// trying to load a module that we've unloaded/abandoned at shutdown.
			if( bIsShutdown )
			{
				ModuleInfo.bWasUnloadedAtShutdown = true;
			}

			// Don't bother firing off events while we're in the middle of shutting down.  These events
			// are designed for subsystems that respond to plugins dynamically being loaded and unloaded,
			// such as the ModuleUI -- but they shouldn't be doing work to refresh at shutdown.
			else
			{
				// A module was successfully unloaded.  Fire callbacks.
				ModulesChangedEvent.Broadcast( InModuleName, EModuleChangeReason::ModuleUnloaded );
			}

			return true;
		}
	}

	return false;
}

void FModuleManager::AbandonModule( const FName InModuleName )
{
	// Do we even know about this module?
	ModuleInfoPtr ModuleInfoPtr = FindModule(InModuleName);
	if( ModuleInfoPtr.IsValid() )
	{
		FModuleInfo& ModuleInfo = *ModuleInfoPtr;

		// Only if already loaded
		if( ModuleInfo.Module.IsValid() )
		{
			// Will offer use-before-ready protection at next reload
			ModuleInfo.bIsReady = false;

			// Allow the module to shut itself down
			ModuleInfo.Module->ShutdownModule();

			// Release reference to module interface.  This will actually destroy the module object.
			// @todo UE4 DLL: Could be dangerous in some cases to reset the module interface while abandoning.  Currently not
			// a problem because script modules don't implement any functionality here.  Possible, we should keep these references
			// alive off to the side somewhere (intentionally leak)
			ModuleInfo.Module.Reset();

			// A module was successfully unloaded.  Fire callbacks.
			ModulesChangedEvent.Broadcast( InModuleName, EModuleChangeReason::ModuleUnloaded );
		}
	}
}

void FModuleManager::UnloadModulesAtShutdown()
{
	ensure(IsInGameThread());

	TRACE_CPUPROFILER_EVENT_SCOPE(UnloadModulesAtShutdown);

	struct FModulePair
	{
		FName ModuleName;
		int32 LoadOrder;
		IModuleInterface* Module;
		FModulePair(FName InModuleName, int32 InLoadOrder, IModuleInterface* InModule)
			: ModuleName(InModuleName)
			, LoadOrder(InLoadOrder)
			, Module(InModule)
		{
			check(LoadOrder > 0); // else was never initialized
		}
		bool operator<(const FModulePair& Other) const
		{
			return LoadOrder > Other.LoadOrder; //intentionally backwards, we want the last loaded module first
		}
	};

	TArray<FModulePair> ModulesToUnload;

	for (const auto& ModuleIt : Modules)
	{
		ModuleInfoRef ModuleInfo( ModuleIt.Value );

		// Only if already loaded
		if( ModuleInfo->Module.IsValid() )
		{
			// Only if the module supports shutting down in this phase
			if( ModuleInfo->Module->SupportsAutomaticShutdown() )
			{
				ModulesToUnload.Emplace(ModuleIt.Key, ModuleIt.Value->LoadOrder, ModuleInfo->Module.Get());
			}
		}
	}

	ModulesToUnload.Sort();

	// Call PreUnloadCallback on all modules first
	for (FModulePair& ModuleToUnload : ModulesToUnload)
	{
		ModuleToUnload.Module->PreUnloadCallback();
		ModuleToUnload.Module = nullptr;
	}

	// Now actually unload all modules
	for (FModulePair& ModuleToUnload : ModulesToUnload)
	{
		UE_LOGF(LogModuleManager, Verbose, "Shutting down and abandoning module %ls (%d)", *ModuleToUnload.ModuleName.ToString(), ModuleToUnload.LoadOrder);
		const bool bIsShutdown = true;
		UnloadModule(ModuleToUnload.ModuleName, bIsShutdown);
		UE_LOGF(LogModuleManager, Verbose, "Returned from UnloadModule.");
	}
}

IModuleInterface* FModuleManager::GetModule( const FName InModuleName )
{
	// Do we even know about this module?
	ModuleInfoPtr ModuleInfo = FindModule(InModuleName);

	if (!ModuleInfo.IsValid())
	{
		return nullptr;
	}

	// For loading purpose, the GameThread is allowed to query modules that are not yet ready
	// bIsReady load acquire (bIsReady is TAtomic) :
	if ( ModuleInfo->bIsReady || IsInGameThread() )
	{
		return ModuleInfo->Module.Get();
	}
	
#if !UE_BUILD_SHIPPING
	// if you hit this it's almost always a bug
	// it means your call to GetModule() is running at the same time that module is loading
	// so you will see null or not depending on timing
	UE_LOGF(LogModuleManager, Warning, "GetModule racing against IsReady: %ls", *InModuleName.ToString());
#endif

	return nullptr;
}

bool FModuleManager::IsModuleSafeToUse(const FName InModuleName) const
{
	FScopeLock Lock(&ModulesCriticalSection);

	if (bIsLoadingDynamicLibrary)
	{
		// Dynamic library loading triggers a lot of construction at a time where we don't have good bookkeeping on modules
		return true;
	}

	if (!ModulePathsCache.Contains(InModuleName))
	{
		// Module is compiled-in
		return true;
	}

#if UE_MERGED_MODULES
	const FString& ModuleLibrary = ModulePathsCache[InModuleName];
	if (LibraryHandles.Contains(ModuleLibrary))
	{
		// The merged library for this module is currently live
		return true;
	}

	// Module is neither compiled-in, nor currently loaded
	return false;

#else

	// Just check if the module is loaded
	return IsModuleLoaded(InModuleName);

#endif // UE_MERGED_MODULES

}

bool FModuleManager::Exec_Dev( UWorld* Inworld, const TCHAR* Cmd, FOutputDevice& Ar )
{
#if !UE_BUILD_SHIPPING
	if ( FParse::Command( &Cmd, TEXT( "Module" ) ) )
	{
		// List
		if( FParse::Command( &Cmd, TEXT( "List" ) ) )
		{
			if( Modules.Num() > 0 )
			{
				Ar.Logf( TEXT( "Listing all %i known modules:\n" ), Modules.Num() );

				TArray< FString > StringsToDisplay;
				for( FModuleMap::TConstIterator ModuleIt( Modules ); ModuleIt; ++ModuleIt )
				{
					StringsToDisplay.Add(
						FString::Printf( TEXT( "    %s [File: %s] [Loaded: %s]" ),
							*ModuleIt.Key().ToString(),
							*ModuleIt.Value()->Filename,
							ModuleIt.Value()->Module.IsValid() != false? TEXT( "Yes" ) : TEXT( "No" ) ) );
				}

				// Sort the strings
				StringsToDisplay.Sort();

				// Display content
				for( TArray< FString >::TConstIterator StringIt( StringsToDisplay ); StringIt; ++StringIt )
				{
					Ar.Log( *StringIt );
				}
			}
			else
			{
				Ar.Logf( TEXT( "No modules are currently known." ), Modules.Num() );
			}

			return true;
		}

#if !IS_MONOLITHIC
		// Load <ModuleName>
		else if( FParse::Command( &Cmd, TEXT( "Load" ) ) )
		{
			const FString ModuleNameStr = FParse::Token( Cmd, 0 );
			if( !ModuleNameStr.IsEmpty() )
			{
				const FName ModuleName( *ModuleNameStr );
				if( !IsModuleLoaded( ModuleName ) )
				{
					Ar.Logf( TEXT( "Loading module" ) );
					LoadModuleWithCallback( ModuleName, Ar );
				}
				else
				{
					Ar.Logf( TEXT( "Module is already loaded." ) );
				}
			}
			else
			{
				Ar.Logf( TEXT( "Please specify a module name to load." ) );
			}

			return true;
		}


		// Unload <ModuleName>
		else if( FParse::Command( &Cmd, TEXT( "Unload" ) ) )
		{
			const FString ModuleNameStr = FParse::Token( Cmd, 0 );
			if( !ModuleNameStr.IsEmpty() )
			{
				const FName ModuleName( *ModuleNameStr );

				if( IsModuleLoaded( ModuleName ) )
				{
					Ar.Logf( TEXT( "Unloading module." ) );
					UnloadOrAbandonModuleWithCallback( ModuleName, Ar );
				}
				else
				{
					Ar.Logf( TEXT( "Module is not currently loaded." ) );
				}
			}
			else
			{
				Ar.Logf( TEXT( "Please specify a module name to unload." ) );
			}

			return true;
		}


		// Reload <ModuleName>
		else if( FParse::Command( &Cmd, TEXT( "Reload" ) ) )
		{
			const FString ModuleNameStr = FParse::Token( Cmd, 0 );
			if( !ModuleNameStr.IsEmpty() )
			{
				const FName ModuleName( *ModuleNameStr );

				if( IsModuleLoaded( ModuleName ) )
				{
					Ar.Logf( TEXT( "Reloading module.  (Module is currently loaded.)" ) );
					UnloadOrAbandonModuleWithCallback( ModuleName, Ar );
				}
				else
				{
					Ar.Logf( TEXT( "Reloading module.  (Module was not loaded.)" ) );
				}

				if( !IsModuleLoaded( ModuleName ) )
				{
					Ar.Logf( TEXT( "Reloading module" ) );
					LoadModuleWithCallback( ModuleName, Ar );
				}
			}

			return true;
		}
#endif // !IS_MONOLITHIC
	}
#endif // !UE_BUILD_SHIPPING

	return false;
}

bool FModuleManager::QueryModule( const FName InModuleName, FModuleStatus& OutModuleStatus ) const
{
	// Do we even know about this module?
	TSharedPtr<const FModuleInfo, ESPMode::ThreadSafe> ModuleInfoPtr = FindModule(InModuleName);

	if (!ModuleInfoPtr.IsValid())
	{
		// Not known to us
		return false;
	}

	OutModuleStatus.Name = InModuleName.ToString();
	OutModuleStatus.FilePath = FPaths::ConvertRelativePathToFull(ModuleInfoPtr->Filename);
	OutModuleStatus.bIsLoaded = ModuleInfoPtr->Module.IsValid();

	if( OutModuleStatus.bIsLoaded )
	{
		OutModuleStatus.bIsGameModule = ModuleInfoPtr->Module->IsGameModule();
	}

	return true;
}

void FModuleManager::QueryModules( TArray< FModuleStatus >& OutModuleStatuses ) const
{
	FScopeLock Lock(&ModulesCriticalSection);
	OutModuleStatuses.Reset(Modules.Num());

	for (const TPair<FName, ModuleInfoRef>& ModuleIt : Modules)
	{
		const FModuleInfo& CurModule = *ModuleIt.Value;

		FModuleStatus ModuleStatus;
		ModuleStatus.Name = ModuleIt.Key.ToString();
		ModuleStatus.FilePath = FPaths::ConvertRelativePathToFull(CurModule.Filename);
		ModuleStatus.bIsLoaded = CurModule.Module.IsValid();

		if( ModuleStatus.bIsLoaded  )
		{
			ModuleStatus.bIsGameModule = CurModule.Module->IsGameModule();
		}

		OutModuleStatuses.Add(MoveTemp(ModuleStatus));
	}
}

#if !IS_MONOLITHIC
FString FModuleManager::GetModuleFilename(FName ModuleName) const
{
	return FindModuleChecked(ModuleName)->Filename;
}

void FModuleManager::SetModuleFilename(FName ModuleName, const FString& Filename)
{
	auto Module = FindModuleChecked(ModuleName);
	Module->Filename = Filename;
	// If it's a new module then also update its OriginalFilename
	if (Module->OriginalFilename.IsEmpty())
	{
		Module->OriginalFilename = Filename;
	}
}

bool FModuleManager::HasAnyOverridenModuleFilename() const
{
	FScopeLock Lock(&ModulesCriticalSection);
	for (const TPair<FName, ModuleInfoRef>& ModuleIt : Modules)
	{
		const FModuleInfo& CurModule = *ModuleIt.Value;
		if(!CurModule.OriginalFilename.IsEmpty() && CurModule.Filename != CurModule.OriginalFilename)
		{
			return true;
		}
	}
	return false;
}

void FModuleManager::SaveCurrentStateForBootstrap(const TCHAR* Filename)
{
	TArray<uint8> FileContent;
	{
		// Use FMemoryWriter because FileManager::CreateFileWriter doesn't serialize FName as string and is not overridable
		FMemoryWriter MemoryWriter(FileContent, true);
		FModuleManager::Get().SerializeStateForBootstrap_Impl(MemoryWriter);
	}

	FFileHelper::SaveArrayToFile(FileContent, Filename);
}

FArchive& operator<<(FArchive& Ar, FModuleManager& ModuleManager)
{
	Ar << ModuleManager.ModulePathsCache;
	Ar << ModuleManager.PendingEngineBinariesDirectories;
	Ar << ModuleManager.PendingGameBinariesDirectories;
	Ar << ModuleManager.EngineBinariesDirectories;
	Ar << ModuleManager.GameBinariesDirectories;
	Ar << ModuleManager.bExtraBinarySearchPathsAdded;
	Ar << ModuleManager.BuildId;
	return Ar;
}

void FModuleManager::SerializeStateForBootstrap_Impl(FArchive& Ar)
{
	// This implementation is meant to stay private and be used for 
	// bootstrapping another processes' module manager with a serialized state.
	// It doesn't include any versioning as it is used with the
	// the same binary executable for both the parent and 
	// children processes. It also takes care or saving/restoring
	// additional dll search directories.
	TArray<FString> DllDirectories;
	if (Ar.IsSaving())
	{
		TMap<FName, FString> OutModulePaths;
		// Force any pending paths to be processed
		FindModulePaths(TEXT("*"), OutModulePaths);

		FPlatformProcess::GetDllDirectories(DllDirectories);
	}

	Ar << *this;
	Ar << DllDirectories;

	if (Ar.IsLoading())
	{
		for (const FString& DllDirectory : DllDirectories)
		{
			FPlatformProcess::AddDllDirectory(*DllDirectory);
		}
	}
}
#endif

void FModuleManager::ResetModulePathsCache()
{
	ModulePathsCache.Reset();

	// Set all folders as pending again
	PendingEngineBinariesDirectories.Append(MoveTemp(EngineBinariesDirectories));
	PendingGameBinariesDirectories  .Append(MoveTemp(GameBinariesDirectories));
}

#if !IS_MONOLITHIC
void FModuleManager::FindModulePaths(const TCHAR* NamePattern, TMap<FName, FString> &OutModulePaths) const
{
	if (ModulePathsCache.Num() == 0)
	{
		// Figure out the BuildId if it's not already set.
		if (!BuildId.IsSet())
		{
			FString FileName = FModuleManifest::GetFileName(FPlatformProcess::GetModulesDirectory(), false);

			FModuleManifest Manifest;
			if (!FModuleManifest::TryRead(FileName, Manifest))
			{
				UE_LOGF(LogModuleManager, Fatal, "Unable to read module manifest from '%ls'. Module manifests are generated at build time, and must be present to locate modules at runtime.", *FileName)
			}

			BuildId = Manifest.BuildId;
		}

		// Add the engine directory to the cache - only needs to be cached once as the contents do not change at runtime
		FindModulePathsInDirectory(FPlatformProcess::GetModulesDirectory(), false, ModulePathsCache);

#if !WITH_EDITOR
		// DebugGame is a hybrid configuration where the engine is in Development and the game Debug.
		// As such, it generates two separate module manifests, one for the engine modules (Development) and
		// one for the game (Debug). We need to load both as otherwise some modules won't be found.
		// We exclude this code in the editor as it handles the game's libraries separately via PendingGameBinariesDirectories.
		if (FApp::GetBuildConfiguration() == EBuildConfiguration::DebugGame)
		{
			FindModulePathsInDirectory(FPlatformProcess::GetModulesDirectory(), true, ModulePathsCache);
		}
#endif
	}

	// If any entries have been added to the PendingEngineBinariesDirectories or PendingGameBinariesDirectories arrays, add any
	// new valid modules within them to the cache.
	// Static iterators used as once we've cached a BinariesDirectories array entry we don't want to do it again (ModuleManager is a Singleton)

	// Search any engine directories
	if (PendingEngineBinariesDirectories.Num() > 0)
	{
		TArray<FString> LocalPendingEngineBinariesDirectories = MoveTemp(PendingEngineBinariesDirectories);
		check(PendingEngineBinariesDirectories.Num() == 0);

		for (const FString& EngineBinaryDirectory : LocalPendingEngineBinariesDirectories)
		{
			FindModulePathsInDirectory(EngineBinaryDirectory, false, ModulePathsCache);
		}

		EngineBinariesDirectories.Append(MoveTemp(LocalPendingEngineBinariesDirectories));
	}

	// Search any game directories
	if (PendingGameBinariesDirectories.Num() > 0)
	{
		TArray<FString> LocalPendingGameBinariesDirectories = MoveTemp(PendingGameBinariesDirectories);
		check(PendingGameBinariesDirectories.Num() == 0);

		for (const FString& GameBinaryDirectory : LocalPendingGameBinariesDirectories)
		{
			FindModulePathsInDirectory(GameBinaryDirectory, true, ModulePathsCache);
		}

		GameBinariesDirectories.Append(MoveTemp(LocalPendingGameBinariesDirectories));
	}

	// Wildcard for all items
	if (FCString::Strcmp(NamePattern, TEXT("*")) == 0)
	{
		OutModulePaths = ModulePathsCache;
		return;
	}

	// Avoid wildcard pattern matching if possible
	if (FCString::Strchr(NamePattern, '*') == nullptr)
	{
		FName Key(NamePattern, FNAME_Find);
		if (Key != FName())
		{
			if (const FString* Value = ModulePathsCache.Find(Key))
			{
				OutModulePaths.Add(Key, *Value);
			}
		}
	}
	else
	{
		// Search the cache
		FString KeyTemp;
		KeyTemp.Reserve(256);
		for (const TPair<FName, FString>& Pair : ModulePathsCache)
		{
			Pair.Key.ToString(KeyTemp);
			if (KeyTemp.MatchesWildcard(NamePattern))
			{
				OutModulePaths.Add(Pair.Key, Pair.Value);
			}
		}
	}
}

void FModuleManager::FindModulePathsInDirectory(const FString& InDirectoryName, bool bIsGameDirectory, TMap<FName, FString>& OutModulePaths) const
{
	// Find all the directories to search through, including the base directory
	TArray<FString> SearchDirectoryNames;
	IFileManager::Get().FindFilesRecursive(SearchDirectoryNames, *InDirectoryName, TEXT("*"), false, true);
	SearchDirectoryNames.Insert(InDirectoryName, 0);

	// Enumerate the modules in each directory
	for (const FString& SearchDirectoryName : SearchDirectoryNames)
	{
		FModuleManifest Manifest;
		FString Filename = FModuleManifest::GetFileName(SearchDirectoryName, bIsGameDirectory);
		if (FModuleManifest::TryRead(Filename, Manifest))
		{
			if ((Manifest.BuildId == BuildId.GetValue() || SearchDirectoryName.Contains(TEXT("/Engine/Plugins/Bridge/"))))
			{
#if UE_MERGED_MODULES
				LibraryDependencies = Manifest.LibraryDependencies;
#endif // UE_MERGED_MODULES

				for (const TPair<FString, FString>& Pair : Manifest.ModuleNameToFileName)
				{
					OutModulePaths.Add(FName(*Pair.Key), *FPaths::Combine(*SearchDirectoryName, *Pair.Value));
				}
			}
			else
			{
				UE_LOGFMT(LogModuleManager, Log, "Skipping out-of-date modules in manifest '{Filename}' (BuildId {ModuleBuildId} != {BuildId}):",
					Filename, Manifest.BuildId, BuildId.GetValue());
				for (const TPair<FString, FString>& Pair : Manifest.ModuleNameToFileName)
				{
					UE_LOGFMT(LogModuleManager, Log, "    Skipping module '{Filename}'.", *FPaths::Combine(*SearchDirectoryName, *Pair.Value));
				}
			}
		}
	}
}
#endif

void FModuleManager::ProcessPendingStaticallyLinkedModuleInitializers() const
{
	if (PendingStaticallyLinkedModuleInitializers.Num() == 0)
	{
		return;
	}

	for (TPair<FLazyName, FInitializeStaticallyLinkedModule>& Module : PendingStaticallyLinkedModuleInitializers)
	{
		UE_LOGF(LogModuleManager, VeryVerbose, "ProcessPendingStaticallyLinkedModuleInitializers: '%ls'",
			*FName(Module.Key).ToString());

		FName NameKey = FName(Module.Key);
		checkf(!StaticallyLinkedModuleInitializers.Contains(NameKey), TEXT("Duplicate module '%s' registered"), *NameKey.ToString());
		StaticallyLinkedModuleInitializers.Add(NameKey, Module.Value);
	}

	PendingStaticallyLinkedModuleInitializers.Empty();
}

void FModuleManager::UnloadOrAbandonModuleWithCallback(const FName InModuleName, FOutputDevice &Ar)
{
	auto Module = FindModuleChecked(InModuleName);
	
	Module->Module->PreUnloadCallback();

	const bool bIsHotReloadable = DoesLoadedModuleHaveUObjects( InModuleName );
	if (bIsHotReloadable && Module->Module->SupportsDynamicReloading())
	{
		if( !UnloadModule( InModuleName ))
		{
			Ar.Logf(TEXT("Module couldn't be unloaded, and so can't be recompiled while the engine is running."));
		}
	}
	else
	{
		// Don't warn if abandoning was the intent here
		Ar.Logf(TEXT("Module being reloaded does not support dynamic unloading -- abandoning existing loaded module so that we can load the recompiled version!"));
		AbandonModule( InModuleName );
	}

	// Ensure module is unloaded
	check(!IsModuleLoaded(InModuleName));
}


void FModuleManager::AbandonModuleWithCallback(const FName InModuleName)
{
	auto Module = FindModuleChecked(InModuleName);
	
	Module->Module->PreUnloadCallback();

	AbandonModule( InModuleName );

	// Ensure module is unloaded
	check(!IsModuleLoaded(InModuleName));
}


bool FModuleManager::LoadModuleWithCallback( const FName InModuleName, FOutputDevice &Ar )
{
	IModuleInterface* LoadedModule = LoadModule( InModuleName );
	if (!LoadedModule)
	{
		Ar.Logf(TEXT("Module couldn't be loaded."));
		return false;
	}

	LoadedModule->PostLoadCallback();
	return true;
}

void FModuleManager::AddExtraBinarySearchPaths()
{
	if (!bExtraBinarySearchPathsAdded)
	{
		// Ensure that dependency dlls can be found in restricted sub directories
		TArray<FString> RestrictedFolderNames = { TEXT("NoRedist"), TEXT("NotForLicensees"), TEXT("CarefullyRedist"), TEXT("LimitedAccess") };
		for (FName PlatformName : FDataDrivenPlatformInfoRegistry::GetConfidentialPlatforms())
		{
			RestrictedFolderNames.Add(PlatformName.ToString());
		}

		FString ModuleDir = FPlatformProcess::GetModulesDirectory();
		for (const FString& RestrictedFolderName : RestrictedFolderNames)
		{
			FString RestrictedFolder = ModuleDir / RestrictedFolderName;
			if (FPaths::DirectoryExists(RestrictedFolder))
			{
				AddBinariesDirectory(*RestrictedFolder, false);
			}
		}

		bExtraBinarySearchPathsAdded = true;
	}
}

void FModuleManager::MakeUniqueModuleFilename( const FName InModuleName, FString& UniqueSuffix, FString& UniqueModuleFileName ) const
{
	// NOTE: Formatting of the module file name must match with the code in HotReload.cs, ReplaceSuffix.

	TSharedRef<const FModuleInfo, ESPMode::ThreadSafe> Module = FindModuleChecked(InModuleName);

	IFileManager& FileManager = IFileManager::Get();

	do
	{
		// Use a random number as the unique file suffix, but mod it to keep it of reasonable length
		UniqueSuffix = FString::Printf( TEXT("%04d"), FMath::Rand() % 10000 );

		const FString ModuleName = InModuleName.ToString();
		const int32 MatchPos = Module->OriginalFilename.Find(ModuleName, ESearchCase::IgnoreCase, ESearchDir::FromEnd);

		if (MatchPos != INDEX_NONE)
		{
			const int32 SuffixPos = MatchPos + ModuleName.Len();
			UniqueModuleFileName = FString::Printf( TEXT( "%s-%s%s" ),
				*Module->OriginalFilename.Left( SuffixPos ),
				*UniqueSuffix,
				*Module->OriginalFilename.Right( Module->OriginalFilename.Len() - SuffixPos ) );
		}
	}
	while (FileManager.GetFileAgeSeconds(*UniqueModuleFileName) != -1.0);
}

const TCHAR *FModuleManager::GetUBTConfiguration()
{
	return LexToString(FApp::GetBuildConfiguration());
}

void FModuleManager::RegisterStaticallyLinkedModule(const FLazyName InModuleName, FInitializeStaticallyLinkedModuleRaw* InInitializer)
{
	Get().PendingStaticallyLinkedModuleInitializers.Emplace(InModuleName, FInitializeStaticallyLinkedModule::CreateStatic(InInitializer));
}

void FModuleManager::StartProcessingNewlyLoadedObjects()
{
	UE_LOGFMT(LogModuleManager, Log, "StartProcessingNewlyLoadedObjects");
	// Only supposed to be called once
	ensure( bCanProcessNewlyLoadedObjects == false );	
	bCanProcessNewlyLoadedObjects = true;
}


void FModuleManager::AddBinariesDirectory(const TCHAR *InDirectory, bool bIsGameDirectory)
{
	if (bIsGameDirectory)
	{
		PendingGameBinariesDirectories.Add(InDirectory);
	}
	else
	{
		PendingEngineBinariesDirectories.Add(InDirectory);
	}

	FPlatformProcess::AddDllDirectory(InDirectory);

	// Also recurse into restricted sub-folders, if they exist
	const TCHAR* RestrictedFolderNames[] = { TEXT("NoRedist"), TEXT("NotForLicensees"), TEXT("CarefullyRedist"), TEXT("LimitedAccess") };
	for (const TCHAR* RestrictedFolderName : RestrictedFolderNames)
	{
		FString RestrictedFolder = FPaths::Combine(InDirectory, RestrictedFolderName);
		if (FPaths::DirectoryExists(RestrictedFolder))
		{
			AddBinariesDirectory(*RestrictedFolder, bIsGameDirectory);
		}
	}
}

void FModuleManager::LoadModuleBinaryOnly(FName ModuleName)
{
#if !IS_MONOLITHIC
	TMap<FName, FString> ModulePaths;
	FindModulePaths(*ModuleName.ToString(), ModulePaths);
	if (ModulePaths.Num() == 1)
	{
		FString ModuleFilename = MoveTemp(TMap<FName, FString>::TIterator(ModulePaths).Value());
		InternalLoadLibrary(ModuleName, ModuleFilename);
	}
#endif
}

void FModuleManager::SetGameBinariesDirectory(const TCHAR* InDirectory)
{
#if !IS_MONOLITHIC
	// Before loading game DLLs, make sure that the DLL files can be located by the OS by adding the
	// game binaries directory to the OS DLL search path.  This is so that game module DLLs which are
	// statically loaded as dependencies of other game modules can be located by the OS.
	FPlatformProcess::PushDllDirectory(InDirectory);

	// Add it to the list of game directories to search
	PendingGameBinariesDirectories.Add(InDirectory);
#endif
}

FString FModuleManager::GetGameBinariesDirectory() const
{
	if (GameBinariesDirectories.Num())
	{
		return GameBinariesDirectories[0];
	}
	if (PendingGameBinariesDirectories.Num())
	{
		return PendingGameBinariesDirectories[0];
	}
	return FString();
}

bool FModuleManager::DoesLoadedModuleHaveUObjects( const FName ModuleName ) const
{
	if (IsModuleLoaded(ModuleName) && IsPackageLoaded.IsBound())
	{
		return IsPackageLoaded.Execute(*FString(FString("/Script/") + ModuleName.ToString()));
	}

	return false;
}

int32 FModuleManager::GetModuleCount() const
{
	// Theoretically thread safe but by the time we return new modules could've been added
	// so no point in locking here. ModulesCriticalSection should be locked by the caller
	// if it wants to rely on the returned value.
	return Modules.Num();
}

namespace
{
	EActiveReloadType GActiveReloadType = EActiveReloadType::None;
	IReload* GActiveReloadInterface = nullptr;
}

#if WITH_RELOAD
EActiveReloadType GetActiveReloadType()
{
#if WITH_HOT_RELOAD
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (GIsHotReload)
	{
		check(GActiveReloadInterface);
		return EActiveReloadType::HotReload;
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif

	return GActiveReloadType;
}

void BeginReload(EActiveReloadType ActiveReloadType, IReload& Interface)
{
	check(GActiveReloadInterface == nullptr);
#if WITH_HOT_RELOAD
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	GIsHotReload = ActiveReloadType == EActiveReloadType::HotReload;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif

	GActiveReloadType = ActiveReloadType;
	GActiveReloadInterface = &Interface;
}

void EndReload()
{
#if WITH_HOT_RELOAD
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	GIsHotReload = false;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif
	GActiveReloadType = EActiveReloadType::None;
	GActiveReloadInterface = nullptr;
}

IReload* GetActiveReloadInterface()
{
	return GActiveReloadInterface;
}

bool IsReloadActive()
{
	return GetActiveReloadType() != EActiveReloadType::None;
}
#endif
