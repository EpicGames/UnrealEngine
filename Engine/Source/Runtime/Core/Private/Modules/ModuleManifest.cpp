// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManifest.h"
#include "Misc/FileHelper.h"
#include "Misc/App.h"
#include "Misc/StringBuilder.h"
#include "Modules/ModuleManager.h"
#include "Modules/SimpleParse.h"

FModuleManifest::FModuleManifest()
{
}

FString FModuleManifest::GetFileName(const FString& DirectoryName, bool bIsGameFolder)
{
#if UE_BUILD_DEVELOPMENT
	return DirectoryName / ((FApp::GetBuildConfiguration() == EBuildConfiguration::DebugGame && bIsGameFolder)? TEXT(UBT_MODULE_MANIFEST_DEBUGGAME) : TEXT(UBT_MODULE_MANIFEST));
#else
	return DirectoryName / TEXT(UBT_MODULE_MANIFEST);
#endif
}

// Assumes GetTypeHash(AltKeyType) matches GetTypeHash(KeyType)
template<class KeyType, class ValueType, class AltKeyType, class AltValueType>
ValueType& FindOrAddHeterogeneous(TMap<KeyType, ValueType>& Map, const AltKeyType& Key, const AltValueType& Value) 
{
	checkSlow(GetTypeHash(KeyType(Key)) == GetTypeHash(Key));
	ValueType* Existing = Map.FindByHash(GetTypeHash(Key), Key);
	return Existing ? *Existing : Map.Emplace(KeyType(Key), AltValueType(Value));
}

bool FModuleManifest::TryRead(const FString& FileName, FModuleManifest& OutManifest)
{
	// Read the file to a string
	FString Text;
	if (!FFileHelper::LoadFileToString(Text, *FileName))
	{
		return false;
	}

	const TCHAR* Ptr = *Text;

	// Helpers
	const auto Whitespace = [&Ptr]()
	{
		return FSimpleParse::MatchZeroOrMoreWhitespace(Ptr);
	};
	const auto ObjectStart = [&Ptr]()
	{
		return FSimpleParse::MatchChar(Ptr, TEXT('{'));
	};
	const auto ObjectEnd = [&Ptr]()
	{
		return FSimpleParse::MatchChar(Ptr, TEXT('}'));
	};
	const auto ArrayStart = [&Ptr]()
	{
		return FSimpleParse::MatchChar(Ptr, TEXT('['));
	};
	const auto ArrayEnd = [&Ptr]()
	{
		return FSimpleParse::MatchChar(Ptr, TEXT(']'));
	};
	const auto Colon = [&Ptr]()
	{
		return FSimpleParse::MatchChar(Ptr, TEXT(':'));
	};
	const auto Comma = [&Ptr]()
	{
		return FSimpleParse::MatchChar(Ptr, TEXT(','));
	};

	if (!Whitespace() || !ObjectStart())
	{
		return false;
	}

	if (!Whitespace() || ObjectEnd())
	{
		return false;
	}

	for (;;)
	{
		TStringBuilder<64> Field;
		if (!FSimpleParse::ParseString(Ptr, Field))
		{
			return false;
		}

		if (!Whitespace() || !Colon())
		{
			return false;
		}

		if (!Whitespace())
		{
			return false;
		}

		if (Field.ToView() == TEXTVIEW("BuildId"))
		{
			if (!FSimpleParse::ParseString(Ptr, OutManifest.BuildId))
			{
				return false;
			}
		}

		// Modules is an array of ModuleName:ModulePath pairs
		else if (Field.ToView() == TEXTVIEW("Modules"))
		{
			if (!Whitespace() || !ObjectStart())
			{
				return false;
			}

			if (!Whitespace())
			{
				return false;
			}

			if (!ObjectEnd())
			{
				for (;;)
				{
					TStringBuilder<64> ModuleName;
					TStringBuilder<80> ModulePath;
					if (!FSimpleParse::ParseString(Ptr, ModuleName) || !Whitespace() || !Colon() || !Whitespace() || !FSimpleParse::ParseString(Ptr, ModulePath) || !Whitespace())
					{
						return false;
					}

					FindOrAddHeterogeneous(OutManifest.ModuleNameToFileName, ModuleName.ToView(), ModulePath.ToView());

					if (ObjectEnd())
					{
						break;
					}

					if (!Comma() || !Whitespace())
					{
						return false;
					}
				}
			}
		}

		// LibraryDependencies is an array of LibraryName keys to arrays of DependencyName values
		else if (Field.ToView() == TEXTVIEW("LibraryDependencies"))
		{
			if (!Whitespace() || !ObjectStart())
			{
				return false;
			}

			if (!Whitespace())
			{
				return false;
			}

			if (!ObjectEnd())
			{
				for (;;)
				{
					TStringBuilder<64> LibraryName;
					if (!FSimpleParse::ParseString(Ptr, LibraryName) || !Whitespace() || !Colon())
					{
						return false;
					}

					if (!Whitespace() || !ArrayStart())
					{
						return false;
					}

					if (!Whitespace())
					{
						return false;
					}

					OutManifest.LibraryDependencies.Add(LibraryName.ToString(), TArray<FString>());

					for (;;)
					{
						TStringBuilder<64> DependencyName;
						if (!FSimpleParse::ParseString(Ptr, DependencyName) || !Whitespace())
						{
							return false;
						}

						OutManifest.LibraryDependencies[LibraryName.ToString()].Add(DependencyName.ToString());

						if (!Whitespace())
						{
							return false;
						}

						if (ArrayEnd())
						{
							break;
						}

						if (!Comma() || !Whitespace())
						{
							return false;
						}
					}

					if (!Whitespace())
					{
						return false;
					}

					if (ObjectEnd())
					{
						break;
					}

					if (!Comma() || !Whitespace())
					{
						return false;
					}
				}
			}
		}
		else
		{
			return false;
		}

		if (!Whitespace())
		{
			return false;
		}

		if (ObjectEnd())
		{
			return true;
		}

		if (!Comma() || !Whitespace())
		{
			return false;
		}
	}
}
