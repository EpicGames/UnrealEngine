// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/SimpleParse.h"

#include "Containers/UnrealString.h"
#include "Misc/StringBuilder.h"

bool FSimpleParse::MatchZeroOrMoreWhitespace(const TCHAR*& InOutPtr)
{
	const TCHAR* Ptr = InOutPtr;
	while (*Ptr == TEXT(' ') || *Ptr == TEXT('\n') || *Ptr == TEXT('\r') || *Ptr == TEXT('\t'))
	{
		++Ptr;
	}
	InOutPtr = Ptr;
	return true;
}

bool FSimpleParse::MatchChar(const TCHAR*& InOutPtr, TCHAR Ch)
{
	const TCHAR* Ptr = InOutPtr;
	if (*Ptr != Ch)
	{
		return false;
	}

	InOutPtr = Ptr + 1;
	return true;
}

template<class T>
bool SimpleParseString(const TCHAR*& InOutPtr, T& OutStr)
{
	const TCHAR* Ptr = InOutPtr;
	if (*Ptr != '"')
	{
		return false;
	}

	for (;;)
	{
		++Ptr;

		TCHAR Ch = *Ptr;
		switch (Ch)
		{
			case '"':
				InOutPtr = Ptr + 1;
				return true;

			case '\0':
			case '\n':
			case '\r':
			case '\t':
				return false;

			case '\\':
				++Ptr;
				switch (*Ptr)
				{
					case '\\': OutStr += '\\'; break;
					case '\"': OutStr += '\"'; break;
					case '/':  OutStr += '/';  break;
					case 'b':  OutStr += '\b'; break;
					case 'f':  OutStr += '\f'; break;
					case 'n':  OutStr += '\n'; break;
					case 'r':  OutStr += '\r'; break;
					case 't':  OutStr += '\t'; break;

					default:
						return false;
				}
				break;

			default:
				OutStr += Ch;
				break;
		}
	}
}

bool FSimpleParse::ParseString(const TCHAR*& InOutPtr, FString& OutStr)
{
	return SimpleParseString(InOutPtr, OutStr);
}

bool FSimpleParse::ParseString(const TCHAR*& InOutPtr, FStringBuilderBase& OutStr)
{
	return SimpleParseString(InOutPtr, OutStr);
}

bool FSimpleParse::ParseUnsignedNumber(const TCHAR*& InOutPtr, int32& OutNumber)
{
	const TCHAR* Ptr = InOutPtr;

	TCHAR Ch = *Ptr;
	if (Ch < TEXT('0') || Ch > TEXT('9'))
	{
		return false;
	}

	++Ptr;
	int32 Number = Ch - TEXT('0');
	for (;;)
	{
		Ch = *Ptr;

		if (Ch < TEXT('0') || Ch > TEXT('9'))
		{
			InOutPtr = Ptr;
			OutNumber = Number;
			return true;
		}

		Number *= 10;
		Number += Ch - TEXT('0');
		++Ptr;
	}
}
