// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StringFwd.h"
#include "CoreTypes.h"

class FString;

struct FSimpleParse
{
	static bool MatchZeroOrMoreWhitespace(const TCHAR*& InOutPtr);
	static bool MatchChar(const TCHAR*& InOutPtr, TCHAR Ch);
	static bool ParseString(const TCHAR*& InOutPtr, FString& OutStr);
	static bool ParseString(const TCHAR*& InOutPtr, FStringBuilderBase& OutStr);
	static bool ParseUnsignedNumber(const TCHAR*& InOutPtr, int32& OutNumber);
};
