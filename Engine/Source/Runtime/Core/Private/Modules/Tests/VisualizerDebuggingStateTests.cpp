// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"
#include "Modules/VisualizerDebuggingState.h"
#include "HAL/UnrealMemory.h"

#if WITH_DEV_AUTOMATION_TESTS && UE_VISUALIZER_DEBUGGING_STATE

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FCoreDebuggingStateTest, 
		"System.Core.DebuggingState", 
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FCoreDebuggingStateTest::RunTest(const FString& Parameters)
{
	using namespace UE::Core;

	struct FTestData
	{
		FGuid ID;
		const char* String;
		uint32* Pointer;
	};

	struct FTestState : FVisualizerDebuggingState
	{
		EVisualizerDebuggingStateResult Assign(const FGuid& UniqueId, void* DebugPtr)
		{
			return AssignImpl(UniqueId, DebugPtr);
		}
		void* Find(const FGuid& UniqueId) const
		{
			return FVisualizerDebuggingState::Find(UniqueId);
		}
	};

	FTestState TestState;

	uint32 ResultValues[] = {
		1u,
		10u,
		100u,
		1000u,
		2u,
		20u,
		200u,
		2000u
	};

	FTestData Tests[] = {
		{ FGuid(), "93a891b0f3404d9c9b1f51981966e1e0", &ResultValues[0] },
		{ FGuid(), "06a5fe3be35d4d2987abfeaea8c54035", &ResultValues[1] },
		{ FGuid(), "e87d5c5d7f9948d4a051de51ecfb9b25", &ResultValues[2] },
		{ FGuid(), "1233cca4d6ee400cad1ca3f8802ac523", &ResultValues[3] },
	};

	// Try assigning the pointers multiple times with different values each time
	for (int32 Pass = 0; Pass < 2; ++Pass)
	{
		for (int32 TestIndex = 0; TestIndex < UE_ARRAY_COUNT(Tests); ++TestIndex)
		{
			FTestData& Test = Tests[TestIndex];

			Test.Pointer = &ResultValues[TestIndex + Pass*UE_ARRAY_COUNT(Tests)];

			if (!FGuid::ParseExact(Test.String, EGuidFormats::DigitsLower, Test.ID))
			{
				this->AddError(FString::Printf(TEXT("Error parsing GUID string %hs!"), Test.String));
				continue;
			}

			EVisualizerDebuggingStateResult Result = TestState.Assign(Test.ID, Test.Pointer);
			if (Result != EVisualizerDebuggingStateResult::Success)
			{
				this->AddError(FString::Printf(TEXT("There was an error registering %hs was not found!"), Test.String));
			}
		}
	}

	void* ExpectedResults[] = {
		&ResultValues[4],
		&ResultValues[5],
		&ResultValues[6],
		&ResultValues[7],
	};

	for (int32 Index = 0; Index < UE_ARRAY_COUNT(Tests); ++Index)
	{
		const FTestData& Test = Tests[Index];
		void* Result = TestState.Find(Test.ID);
		if (Result == nullptr)
		{
			this->AddError(FString::Printf(TEXT("Debugging pointer for moniker %hs was not found!"), Test.String));
		}
		else if (Result != Test.Pointer || Result != ExpectedResults[Index])
		{
			this->AddError(FString::Printf(TEXT("Debugging pointer for moniker %hs was incorrect!"), Test.String));
		}
	}

	// Test that assigning a fake GUID comprising the others results in a string collision
	static const char* const CollisionGuid = "9b1f51981966e1e006a5fe3be35d4d29";
	FGuid CollisionId;
	if (!FGuid::ParseExact(CollisionGuid, EGuidFormats::DigitsLower, CollisionId))
	{
		this->AddError(FString::Printf(TEXT("Error parsing GUID string %hs!"), CollisionGuid));
		return false;
	}

	if (TestState.Assign(CollisionId, nullptr) != EVisualizerDebuggingStateResult::StringCollision)
	{
		this->AddError(FString::Printf(TEXT("Expected FVisualizerDebuggingState::Assign(\"%hs\") to result in a string collision but it did not"), CollisionGuid));
		return false;
	}

	return true;
}

#endif

