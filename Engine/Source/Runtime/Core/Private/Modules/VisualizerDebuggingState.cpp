// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/VisualizerDebuggingState.h"
#include "CoreGlobals.h"
#include "HAL/UnrealMemory.h"
#include "Containers/StringView.h"
#include "Containers/AnsiString.h"
#include "Algo/IndexOf.h"
#include "Misc/Guid.h"

UE::Core::FVisualizerDebuggingState* GCoreDebuggingState = nullptr;

namespace UE::Core
{

#if UE_VISUALIZER_DEBUGGING_STATE
struct FVisualizerDebuggingStateImpl
{
	/**
	 * Array of GUIDs associated with indices within DebugPtrs
	 */
	TArray<FGuid> UniqueIds;

	/**
	 * Array of user-provided pointers that are associated with the guids in UniqueIds.
	 */
	TArray<void*> DebugPtrs;

	/**
	 * Single string containing all Guids as a string at position Index*32 to support searching with a single strstr instrinsic call, and dividing the result by 32
	 * Example layout:
	 *   OrderedGuidString: "d9ad42709d2c4bc8a2f8f925e1617b288456cfc3222f4833a8afa45a6ed73b5a9873a9fedd8441d49ef6b3258e8a6c60#"
	 *   Ptrs:              [Ptr1                           ,Ptr2                           ,Ptr3]
	 */
	FAnsiString OrderedGuidString;
};
#endif // UE_VISUALIZER_DEBUGGING_STATE

FVisualizerDebuggingState::FVisualizerDebuggingState()
{
#if UE_VISUALIZER_DEBUGGING_STATE
	PimplData = new FVisualizerDebuggingStateImpl;
#endif
}

FVisualizerDebuggingState::~FVisualizerDebuggingState()
{
#if UE_VISUALIZER_DEBUGGING_STATE
	delete PimplData;
#endif
}

#if UE_VISUALIZER_DEBUGGING_STATE

EVisualizerDebuggingStateResult FVisualizerDebuggingState::Assign(const FGuid& UniqueId, void* Ptr)
{
	if (!GCoreDebuggingState)
	{
		static FVisualizerDebuggingState State;
		GCoreDebuggingState = &State;
	}
	return GCoreDebuggingState->AssignImpl(UniqueId, Ptr);
}

void* FVisualizerDebuggingState::Find(const FGuid& UniqueId) const
{
	if (GuidString != nullptr)
	{
		const int32 Index = Algo::IndexOf(PimplData->UniqueIds, UniqueId);
		if (Index != INDEX_NONE)
		{
			return Ptrs[Index];
		}
	}

	return nullptr;
}

EVisualizerDebuggingStateResult FVisualizerDebuggingState::AssignImpl(const FGuid& UniqueId, void* Ptr)
{
	check(UniqueId.IsValid());

	// Locate an existing entry from the string
	const int32 ExistingIndex = Algo::IndexOf(PimplData->UniqueIds, UniqueId);
	if (ExistingIndex != INDEX_NONE)
	{
		Ptrs[ExistingIndex] = Ptr;
		return EVisualizerDebuggingStateResult::Success;
	}

	TAnsiStringBuilder<32> ThisGuidString;
	UniqueId.AppendString(ThisGuidString, EGuidFormats::DigitsLower);

	// Search the existing string for a potential string collision (this is extremely unlikely)
	if (GuidString != nullptr && FCStringAnsi::Strstr(GuidString, *ThisGuidString) != nullptr)
	{
		return EVisualizerDebuggingStateResult::StringCollision;
	}

	// Add a new entry:
	const int32 PreviousNum = PimplData->UniqueIds.Num();
	const int32 NewNum      = PreviousNum + 1;

	// Append the new GUID string to the hasystack string
	PimplData->OrderedGuidString.Append(ThisGuidString);
	PimplData->UniqueIds.Emplace(UniqueId);
	PimplData->DebugPtrs.Emplace(Ptr);

	// Check invariants
	check(PimplData->DebugPtrs.Num()    == PimplData->UniqueIds.Num());
	check(PimplData->DebugPtrs.Num()*32 == PimplData->OrderedGuidString.Len());

	// Update the cached ptrs for direct access inside natvis
	Ptrs = PimplData->DebugPtrs.GetData();
	GuidString = *PimplData->OrderedGuidString;

	return EVisualizerDebuggingStateResult::Success;
}

#endif // UE_VISUALIZER_DEBUGGING_STATE

} // namespace UE::Core