// Copyright Epic Games, Inc. All Rights Reserved.

#include "PosixOS/PosixOSPlatformProcess.h"
#include "Containers/StringConv.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformProcess.h"
#include "Misc/CommandLine.h"
#include "Misc/FileHelper.h"
#include "Misc/Parse.h"
#include <sys/fcntl.h>
#include <sys/file.h>
#include <unistd.h>


static int GFileLockDescriptor = -1;

bool FPosixOSPlatformProcess::IsFirstInstance()
{
	// set default return if we are unable to access lock file.
	static bool bIsFirstInstance = false;
	static bool bNeverFirst = FParse::Param(FCommandLine::Get(), TEXT("neverfirst"));

	if (!bIsFirstInstance && !bNeverFirst)	// once we determined that we're first, this can never change until we exit; otherwise, we re-check each time
	{
		// create the file if it doesn't exist
		if (GFileLockDescriptor == -1)
		{
			FString LockFileName("/tmp/");
			FString ExecPath(FPlatformProcess::ExecutableName());
			ExecPath.ReplaceInline(TEXT("/"), TEXT("-"), ESearchCase::CaseSensitive);
			// [RCL] 2015-09-20: can run out of filename limits (256 bytes) due to a long path, be conservative and assume 4-char UTF-8 name like e.g. Japanese
			ExecPath.RightInline(80, EAllowShrinking::No);

			LockFileName += ExecPath;

			GFileLockDescriptor = open(TCHAR_TO_UTF8(*LockFileName), O_RDWR | O_CREAT, 0666);
		}

		if (GFileLockDescriptor != -1)
		{
			if (flock(GFileLockDescriptor, LOCK_EX | LOCK_NB) == 0)
			{
				// lock file successfully locked by this process - no more checking if we're first!
				bIsFirstInstance = true;
			}
			else
			{
				// we were unable to lock file. so some other process beat us to lock file.
				bIsFirstInstance = false;
			}
		}
	}

	return bIsFirstInstance;
}

void FPosixOSPlatformProcess::CeaseBeingFirstInstance()
{
	if (GFileLockDescriptor != -1)
	{
		// may fail if we didn't have the lock
		flock(GFileLockDescriptor, LOCK_UN | LOCK_NB);
		close(GFileLockDescriptor);
		GFileLockDescriptor = -1;
	}
}


class FPosixOSProcessSentinel : public IProcessSentinel
{
public:
	
	FPosixOSProcessSentinel()
		: fd(-1)
	{
		
	}
	
	virtual ~FPosixOSProcessSentinel()
	{
		close(fd);
		unlink(TCHAR_TO_UTF8(*SavedFilename));
	}
	
	virtual bool CreateSentinelFile(const TCHAR* Name, const FString& Contents) override
	{
		SavedFilename = MakeFilename(Name);

		fd = open(TCHAR_TO_UTF8(*SavedFilename), O_RDWR | O_CREAT, 0666);
		if (flock(fd, LOCK_EX | LOCK_NB) != 0)
		{
			close(fd);
			UE_LOGF(LogHAL, Error, "Another instance is running, unable to create sentinel file %ls. Will not create or remove sentinel in this process", *SavedFilename);
			fd = -1;
			return false;
		}
		
		// write the contents
		FTCHARToUTF8 Convert(*Contents);
		ftruncate(fd, 0);
		write(fd, Convert.Get(), Convert.Length());
		
		UE_LOGF(LogHAL, Display, "Created sentinel %ls", *SavedFilename);				
		return true;
	}
	
	
	
protected:
	virtual bool DoesSentinelFileExistImpl(const TCHAR* Name, FString* OptionalOutContents) override
	{
		FString Filename = MakeFilename(Name);

		// if the file doesn't exist at all, then early out
		if (!IFileManager::Get().FileExists(*Filename))
		{
			return false;
		}
		
		// the file exists, so open a handle to it (not expected to fail)
		int TempFd = open(TCHAR_TO_UTF8(*Filename), O_RDWR, 0666);
		if (TempFd == -1)
		{
			UE_LOGF(LogHAL, Error, "Didn't expect open to fail %ls", *Filename);		
			return false;
		}
		// attempt to get an exclusive lock - if is able to, if it can that means another process
		// had created the file but then died before deleting it. 
		if (flock(TempFd, LOCK_EX | LOCK_NB) == 0)
		{
			close(TempFd);
			// the file shouldn't exist, so clean it up and pretend it never existed
			IFileManager::Get().Delete(*Filename);
			return false;
		}
		close(TempFd);
		
		// get contents if desired, can use normal file ops now
		if (OptionalOutContents)
		{
			FFileHelper::LoadFileToString(*OptionalOutContents, *Filename);
		}
		
		return true;	
	}
	
	// the opened and flock'd file descriptor for this sentinel file
	int fd;
	FString SavedFilename;
};


TSharedPtr<IProcessSentinel> FPosixOSPlatformProcess::CreateProcessSentinelObject()
{
	return TSharedPtr<IProcessSentinel>(new FPosixOSProcessSentinel());
}
