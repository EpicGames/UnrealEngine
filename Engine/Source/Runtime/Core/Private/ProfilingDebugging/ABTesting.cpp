// Copyright Epic Games, Inc. All Rights Reserved.

/**
* Here are a number of profiling helper functions so we do not have to duplicate a lot of the glue
* code everywhere.  And we can have consistent naming for all our files.
*
*/

// Core includes.
#include "ProfilingDebugging/ABTesting.h"
#include "HAL/IConsoleManager.h"

#if ENABLE_ABTEST


#if (UE_BUILD_SHIPPING || UE_BUILD_TEST)
#define ABTEST_LOG(_FormatString_, ...) FPlatformMisc::LowLevelOutputDebugStringf(_FormatString_ TEXT("\n"), ##__VA_ARGS__)
#else
#define ABTEST_LOG(_FormatString_, ...) UE_LOG(LogConsoleResponse, Display, _FormatString_, ##__VA_ARGS__)
#endif


static TAutoConsoleVariable<int32> CVarABTestHistory(
	TEXT("abtest.HistoryNum"),
	1000,
	TEXT("Number of history frames to use for stats."));

static TAutoConsoleVariable<int32> CVarABTestReportNum(
	TEXT("abtest.ReportNum"),
	100,
	TEXT("Number of frames between reports."));

static TAutoConsoleVariable<int32> CVarABTestCoolDown(
	TEXT("abtest.CoolDown"),
	5,
	TEXT("Number of frames to discard data after each command to cover threading."));

static TAutoConsoleVariable<int32> CVarABTestMinFramesPerTrial(
	TEXT("abtest.MinFramesPerTrial"),
	10,
	TEXT("The number of frames to run a given command before switching; this is randomized."));

static TAutoConsoleVariable<int32> CVarABTestNumResamples(
	TEXT("abtest.NumResamples"),
	256,
	TEXT("The number of resamples to use to determine confidence."));



FABTest::FABTest()
		: Stream(9567)
		, bABTestActive(false)
		, bABScopeTestActive(false)
		, bFrameLog(false)
		, ABTestNumSamples(0)
		, RemainingCoolDown(0)
		, CurrentTest(0)
		, RemainingTrial(0)
		, RemainingPrint(0)
		, HistoryNum(0)
		, ReportNum(0)
		, CoolDown(0)
		, MinFramesPerTrial(0)
		, NumResamples(0)
		, TotalScopeTimeInFrame(0)
		, LastGCFrame(0)
	{
	}

void FABTest::StartFrameLog()
{
	TotalTime = 0.0;
	TotalFrames = 0;
	Spikes = 0;
	bFrameLog = true;
	ABTEST_LOG(TEXT("Starting frame log."));
}

void FABTest::FrameLogTick(double Delta)
{
	if (Delta > .034)
	{
		Spikes++;
	}
	else
	{
		TotalFrames++;
		TotalTime += Delta;
	}
	if (TotalFrames > 0 && TotalFrames % 1000 == 0)
	{
		ABTEST_LOG(TEXT("%8d frames   %6.3fms/f    %8d spikes rejected "), TotalFrames, float(1000.0 * TotalTime / TotalFrames), Spikes);
	}
}

const TCHAR* FABTest::TickAndGetCommand()
{
	const TCHAR* OutCommand = nullptr;

	static double LastTime = 0;
	if (bABTestActive && RemainingCoolDown)
	{
		RemainingCoolDown--;
	}
	else if (bABTestActive && LastGCFrame != GLastGCFrame && !bABScopeTestActive) // reject GC frames for whole game tests
	{
		ABTEST_LOG(TEXT("Rejecting abtest frame because of GC."));
	}
	else if (bABTestActive)
	{
		double Delta;
		if (bABScopeTestActive)
		{
			Delta = TotalScopeTimeInFrame;
		}
		else
		{
			Delta = FPlatformTime::Seconds() - LastTime;
			check(Delta > 0.0);
		}

		FSample* Sample = nullptr;
		if (ABTestNumSamples < HistoryNum)
		{
			check(ABTestNumSamples == Samples.Num());
			Sample = &Samples.AddDefaulted_GetRef();
		}
		else
		{
			Sample = &Samples[ABTestNumSamples % HistoryNum];

			// remove the old data
			check(Totals[Sample->TestIndex] > Sample->Micros || bABScopeTestActive);
			Totals[Sample->TestIndex] -= Sample->Micros;
			check(Counts[Sample->TestIndex]);
			Counts[Sample->TestIndex]--;
			check(Sample->InResamples.Num() == NumResamples);
			for (TConstSetBitIterator<> CurIndexIt(Sample->InResamples); CurIndexIt; ++CurIndexIt)
			{
				int32 ResampleIndex = CurIndexIt.GetIndex();
				check(ResampleAccumulators[ResampleIndex] > Sample->Micros || bABScopeTestActive);
				ResampleAccumulators[ResampleIndex] -= Sample->Micros;
				check(ResampleCount[ResampleIndex]);
				ResampleCount[ResampleIndex]--;
			}
		}
		ABTestNumSamples++;
		Sample->Micros = uint32(Delta * 1000000.0f);
		check(Sample->Micros || bABScopeTestActive);
		Sample->TestIndex = CurrentTest;
		Totals[Sample->TestIndex] += Sample->Micros;
		Counts[Sample->TestIndex]++;
		Sample->InResamples.Empty(NumResamples);
		for (int32 ResampleIndex = 0; ResampleIndex < NumResamples; ResampleIndex++)
		{
			bool In = Stream.FRand() > 0.5f;
			Sample->InResamples.Add(In);
			if (In)
			{
				ResampleAccumulators[ResampleIndex] += Sample->Micros;
				ResampleCount[ResampleIndex]++;
			}
		}
		check(RemainingPrint);
		if (!--RemainingPrint)
		{
			if (Counts[0] && Counts[1])
			{
				ABTEST_LOG(TEXT("      %7.4fms  (%4d samples)  A = '%s'"), float(Totals[0]) / float(Counts[0]) / 1000.0f, Counts[0], *ABTestCmds[0]);
				ABTEST_LOG(TEXT("      %7.4fms  (%4d samples)  B = '%s'"), float(Totals[1]) / float(Counts[1]) / 1000.0f, Counts[1], *ABTestCmds[1]);

				float Diff = (float(Totals[0]) / float(Counts[0]) / 1000.0f) - (float(Totals[1]) / float(Counts[1]) / 1000.0f);
				bool bAIsFaster = false;
				if (Diff < 0.0f)
				{
					Diff *= -1.0f;
					bAIsFaster = true;
				}
				static TArray<float> Scores;
				Scores.Reset();
				uint32 TotalFrame = Totals[0] + Totals[1];
				uint32 TotalCount = Counts[0] + Counts[1];
				for (int32 ResampleIndex = 0; ResampleIndex < NumResamples; ResampleIndex++)
				{
					check(TotalFrame > ResampleAccumulators[ResampleIndex] || bABScopeTestActive);
					uint32 Opposite = TotalFrame - ResampleAccumulators[ResampleIndex];
					check(TotalCount > ResampleCount[ResampleIndex] || bABScopeTestActive);
					uint32 OppositeCount = TotalCount - ResampleCount[ResampleIndex];

					// this ABS here gives us better data but makes it one sided, so the 90% confidence interval is at index 80/100 etc
					// it is legit because we could have easily done the inverse permutation
					float SampleDiff = FMath::Abs((float(ResampleAccumulators[ResampleIndex]) / float(ResampleCount[ResampleIndex]) / 1000.0f) - (float(Opposite) / float(OppositeCount) / 1000.0f));
					Scores.Add(SampleDiff);

				}
				Scores.Sort();
				int32 Conf = NumResamples;
				for (int32 Trial = 0; Trial < NumResamples; Trial++)
				{
					if (Scores[Trial] > Diff)
					{
						Conf = Trial;
						break;
					}
				}

				float fConf = 1.0f - (0.5f + float(Conf) / float(NumResamples) / 2.0f);

				if (bAIsFaster)
				{
					ABTEST_LOG(TEXT("      A is %7.4fms faster than B;  %3.0f%% chance this is noise."), Diff, fConf * 100.0f);
				}
				else
				{
					ABTEST_LOG(TEXT("      B is %7.4fms faster than A;  %3.0f%% chance this is noise."), Diff, fConf * 100.0f);
				}

				ABTEST_LOG(TEXT("----------------"));
			}
			else
			{
				ABTEST_LOG(TEXT("No Samples?"));
			}
			RemainingPrint = ReportNum;
		}
		check(RemainingTrial);
		if (!--RemainingTrial)
		{
			OutCommand = SwitchTest(1 - CurrentTest);
		}
	}
	else if (bFrameLog)
	{
		FrameLogTick(FPlatformTime::Seconds() - LastTime);
	}
	LastTime = FPlatformTime::Seconds();

	TotalScopeTimeInFrame = 0.0;
	LastGCFrame = GLastGCFrame;
	return OutCommand;
}

FABTest& FABTest::Get()
{
	static FABTest Singleton;
	return Singleton;
}

void FABTest::ABTestCmdFunc(const TArray<FString>& Args)
{
	FString ABTestCmds[2];
	if (Args.Num() == 1 && Args[0].Compare(FString("framelog"), ESearchCase::IgnoreCase) == 0)
	{
		Get().Stop();
		Get().StartFrameLog();
		return;
	}
	if (Args.Num() == 1 && Args[0].Compare(FString("stop"), ESearchCase::IgnoreCase) == 0)
	{
		Get().Stop();		
		return;
	}
	if (Args.Num() == 1 && Args[0].Compare(FString("scope"), ESearchCase::IgnoreCase) == 0)
	{
		ABTestCmds[0] = FString("ScopeA");
		ABTestCmds[1] = FString("ScopeB");
		Get().Start(ABTestCmds, true);
		return;
	}
	if (Args.Num() == 3 && !Args[0].StartsWith(TEXT("\"")))
	{
		ABTestCmds[0] = Args[0].TrimQuotes() + TEXT(" ") + Args[1].TrimQuotes();
		ABTestCmds[1] = Args[0].TrimQuotes() + TEXT(" ") + Args[2].TrimQuotes();
	}
	else if (Args.Num() > 2 && Args[0].StartsWith(TEXT("\"")))
	{
		FString Work;
		int32 Test = 0;
		for (int32 Index = 0; Index < Args.Num(); Index++)
		{
			Work += Args[Index];
			if (Work.Len() > 2 && Work.StartsWith(TEXT("\"")) && Work.EndsWith(TEXT("\"")))
			{
				ABTestCmds[Test++] = Work.TrimQuotes();
				Work.Empty();
				if (Test > 1)
				{
					break;
				}
			}
			else if (Index + 1 < Args.Num())
			{
				Work += TEXT(" ");
			}
		}
	}
	else if (Args.Num() == 2)
	{
		ABTestCmds[0] = Args[0].TrimQuotes();
		ABTestCmds[1] = Args[1].TrimQuotes();
	}
	else
	{
		ABTEST_LOG(TEXT("abtest command requires two (quoted) arguments or three args or 'stop' or 'scope'."));
		ABTEST_LOG(TEXT("Example: abtest \"r.MyCVar 0\" \"r.MyCVar 1\""));
		ABTEST_LOG(TEXT("Example: abtest r.MyCVar 0 1"));
		return;
	}
	Get().Start(ABTestCmds, false);
}

void FABTest::Stop()
{
	if (bABTestActive)
	{
		ABTEST_LOG(TEXT("Running 'A' console command and stopping test."));
		SwitchTest(0);
		bABTestActive = false;
	}
	else if (bFrameLog)
	{
		ABTEST_LOG(TEXT("Stopping frame log."));
		bFrameLog = false;
	}
	bABScopeTestActive = false;
}

void FABTest::Start(FString* InABTestCmds, bool bScopeTest)
{
	if (bABTestActive)
	{
		Stop();
	}

	if (InABTestCmds)
	{
		ABTestCmds[0] = InABTestCmds[0];
		ABTestCmds[1] = InABTestCmds[1];
	}
	else
	{
		ABTestCmds[0] = FString();
		ABTestCmds[1] = FString();
	}
	
	bABScopeTestActive = bScopeTest;

	HistoryNum = CVarABTestHistory.GetValueOnGameThread();
	ReportNum = CVarABTestReportNum.GetValueOnGameThread();
	CoolDown = CVarABTestCoolDown.GetValueOnGameThread();
	MinFramesPerTrial = CVarABTestMinFramesPerTrial.GetValueOnGameThread();
	NumResamples = CVarABTestNumResamples.GetValueOnGameThread();

	Samples.Empty(HistoryNum);
	ResampleAccumulators.Empty(NumResamples);
	ResampleAccumulators.AddZeroed(NumResamples);
	ResampleCount.Empty(NumResamples);
	ResampleCount.AddZeroed(NumResamples);
	ABTestNumSamples = 0;
	Totals[0] = 0;
	Totals[1] = 0;
	Counts[0] = 0;
	Counts[1] = 0;
	RemainingPrint = ReportNum;

	bABTestActive = true;
	SwitchTest(0);
	ABTEST_LOG(TEXT("abtest started with A = '%s' and B = '%s'"), *ABTestCmds[0], *ABTestCmds[1]);
}

const TCHAR* FABTest::SwitchTest(int32 Index)
{
	RemainingCoolDown = CoolDown;
	CurrentTest = Index;
	RemainingTrial = Stream.RandRange(MinFramesPerTrial, MinFramesPerTrial * 3);
	check(RemainingTrial);

	if (!bABScopeTestActive)
	{
		return *(ABTestCmds[CurrentTest]);
	}
	return nullptr;
}

static FAutoConsoleCommand ABTestCmd(
	TEXT("abtest"),
	TEXT("Provide two console commands or 'stop' to stop the abtest. Frames are timed with the two options, logging results over time."),
	FConsoleCommandWithArgsDelegate::CreateStatic(&FABTest::ABTestCmdFunc)
	);

#endif

