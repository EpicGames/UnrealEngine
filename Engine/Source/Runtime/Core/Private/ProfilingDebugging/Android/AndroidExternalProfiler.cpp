// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "HAL/PlatformMisc.h"
#include "HAL/PlatformProcess.h"
#include "Misc/AssertionMacros.h"
#include "ProfilingDebugging/ExternalProfiler.h"
#include "Features/IModularFeatures.h"
#include "Templates/UniquePtr.h"
#include "Misc/Paths.h"
#include "HAL/FileManager.h"
#include "Android/AndroidProfiler.h"
#include "HAL/PlatformFileManager.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "Misc/CommandLine.h"
#include "Android/AndroidPlatformFile.h"

#if ANDROID_PROFILING_ENABLED && UE_EXTERNAL_PROFILING_ENABLED

namespace PerfettoExternalProfiler
{
	FString StartProfileInternal(const FString& ProfilerArgs)
	{		
		FString ActiveProfileName = FAndroidProfiler::StartCapture(ProfilerArgs, [](const FAndroidProfiler::FProfileResults& Results)
			{
				if (!Results.FilePath.IsEmpty())
				{
					const FString FilenameOnly = FPaths::GetCleanFilename(Results.FilePath);
					const FString DestPath = FPaths::ProfilingDir() / FilenameOnly;
					UE_CLOGF(!Results.Error.IsEmpty(), LogProfilingDebugging, Warning, "Profile %ls completed with log: %ls", *Results.ProfileName, *Results.Error);
					IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
					// move the file to public path so adb can get to it.
					bool bFileMoved = PlatformFile.MoveFile(*DestPath, *Results.FilePath);
					if (!bFileMoved)
					{
						// copy the file if the move fails.
						bFileMoved = PlatformFile.CopyFile(*DestPath, *Results.FilePath);
						PlatformFile.DeleteFile(*Results.FilePath);
					}
					IAndroidPlatformFile& AndroidPlatformFile = IAndroidPlatformFile::GetPlatformPhysical();

					const FString DestAbsolutePath = AndroidPlatformFile.FileRootPath(*DestPath);
					UE_LOGF(LogProfilingDebugging, Log, "Profile %ls complete, retrieve via 'adb pull %ls'", *Results.ProfileName, *DestAbsolutePath);
				}
				else
				{
					UE_LOGF(LogProfilingDebugging, Error, "Profile %ls failed: %ls", *Results.ProfileName, *Results.Error);
				}
			});
		return ActiveProfileName;
	}

	void CancelProfilerInternal(const FString& ProfileName)
	{
		FAndroidProfiler::StopCapture(ProfileName);
	}
}

/**
 * Android perfetto implementation of FExternalProfiler
 */
class FPerfettoExternalProfiler : public FExternalProfiler
{
	static inline FCriticalSection ProfilerCS;

public:

	/** Constructor */
	FPerfettoExternalProfiler()
	{
		// Register as a modular feature
		IModularFeatures::Get().RegisterModularFeature( FExternalProfiler::GetFeatureName(), this );
	}

	/** Destructor */
	virtual ~FPerfettoExternalProfiler()
	{
		IModularFeatures::Get().UnregisterModularFeature( FExternalProfiler::GetFeatureName(), this );
	}

	/** Gets the name of this profiler as a string.  This is used to allow the user to select this profiler in a system configuration file or on the command-line */
	virtual const TCHAR* GetProfilerName() const override
	{
		return TEXT( "AndroidPerfetto" );
	}

	/** Pauses profiling. */
	virtual void ProfilerPauseFunction() override
	{
		FScopeLock Lock(&ProfilerCS);
		FAndroidProfiler::StopCapture( ActiveProfileName );
	}

	/** Resumes profiling. */
	virtual void ProfilerResumeFunction() override
	{
		FScopeLock Lock(&ProfilerCS);
		ActiveProfileName = PerfettoExternalProfiler::StartProfileInternal( ProfilerArgs );
	}

	virtual void Register()
	{
		FString CmdLineProfileArgsOverride;
		if (FParse::Value(FCommandLine::Get(), TEXT("-androidprofilerargs="), CmdLineProfileArgsOverride))
		{
			ProfilerArgs = CmdLineProfileArgsOverride;
		}

		if (FParse::Value(FCommandLine::Get(), TEXT("-androidprofilercsvargs="), CmdLineProfileArgsOverride))
		{
			CSVProfilerArgs = CmdLineProfileArgsOverride;
		}

		FString FrameRange;
        bool bEnableCSVProfiler = FParse::Param(FCommandLine::Get(), TEXT("-csvExtProfCpu"));
		if (FParse::Value(FCommandLine::Get(), TEXT("-csvExtProfCpu="), FrameRange))
		{
			FString Start, End;
			if (FrameRange.Split(TEXT(":"), &Start, &End))
			{
				FrameStart = FCString::Atoi(*Start);
				FrameEnd = FCString::Atoi(*End);
			}
            bEnableCSVProfiler = true;
		}
		
		if (bEnableCSVProfiler)
		{
#if CSV_PROFILER
			OnCSVStartCaptureHandle = FCsvProfiler::Get()->OnCSVProfileStart().AddRaw(this, &FPerfettoExternalProfiler::StartDelegate);
			OnCSVStopCaptureHandle = FCsvProfiler::Get()->OnCSVProfileEnd().AddRaw(this, &FPerfettoExternalProfiler::EndDelegate);
#endif // CSV_PROFILER
		}
	}

	void StartDelegate()
	{
		if (CSVCaptureMode == ECSVCaptureMode::None)
		{
			CSVCaptureMode = ECSVCaptureMode::Start;
		}
	}

	void EndDelegate()
	{
		FScopeLock Lock(&ProfilerCS);
		if (CSVCaptureMode != ECSVCaptureMode::None)
		{
			CSVCaptureMode = ECSVCaptureMode::Stop;
		}
	}

	virtual void FrameSync() override
	{
#if CSV_PROFILER
		switch (CSVCaptureMode)
		{
			case ECSVCaptureMode::Start:
			{
				if (FCsvProfiler::Get()->GetCaptureFrameNumber() >= FrameStart)
				{
					FScopeLock Lock(&ProfilerCS);
					CSVActiveProfileName = PerfettoExternalProfiler::StartProfileInternal(CSVProfilerArgs);
					CSVCaptureMode = ECSVCaptureMode::InProgress;
				}
			}
			break;
			case ECSVCaptureMode::InProgress:
			{
				if (FCsvProfiler::Get()->IsCapturing() && FCsvProfiler::Get()->GetCaptureFrameNumber() >= FrameEnd)
				{
					CSVCaptureMode = ECSVCaptureMode::Stop;
				}
			}
			break;
			case ECSVCaptureMode::Stop:
			{
				FScopeLock Lock(&ProfilerCS);
				FAndroidProfiler::StopCapture(CSVActiveProfileName);
				CSVCaptureMode = ECSVCaptureMode::None;
			}
			break;
			default:
			break;
		}
#endif // CSV_PROFILER
	}

	/**
	 */
	bool Initialize()
	{
		ProfilerArgs = TEXT("system duration=100 buffersize=10000");
		// perfetto requires a duration, we ask for a long duration and expect to cancel within that period.
		CSVProfilerArgs = TEXT("system duration=1000 buffersize=10000 profilename=csvprofile");
		return true;
	}

private:
	FString ProfilerArgs;
	FString ActiveProfileName;

	// CSV profile session params.
	// note the CSV session can run concurrently with ProfilerResumeFunction etc.
	enum class ECSVCaptureMode { None, Start, InProgress, Stop};
	ECSVCaptureMode CSVCaptureMode = ECSVCaptureMode::None;
	FString CSVProfilerArgs;
	FString CSVActiveProfileName;
	int32 FrameStart = 0;
	int32 FrameEnd = 0xFFFFFFFF;

	FDelegateHandle OnCSVStartCaptureHandle;
	FDelegateHandle OnCSVStopCaptureHandle;
};


namespace PerfettoExternalProfiler
{
	struct FAtModuleInit
	{
		FAtModuleInit()
		{
			static TUniquePtr<FPerfettoExternalProfiler> PerfettoExternal = MakeUnique<FPerfettoExternalProfiler>();
			if( !PerfettoExternal->Initialize() )
			{
				PerfettoExternal.Reset();
			}
		}
	};

	static FAtModuleInit AtModuleInit;
}

#endif	// ANDROID_PROFILING_ENABLED && UE_EXTERNAL_PROFILING_ENABLED
