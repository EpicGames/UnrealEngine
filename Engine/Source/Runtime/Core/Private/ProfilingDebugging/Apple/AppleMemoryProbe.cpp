// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProfilingDebugging/MemoryTrace.h"
#include "HAL/IConsoleManager.h"
#include "HAL/Thread.h"
#include "HAL/PlatformProcess.h"
#include "HAL/PlatformAffinity.h"
#include "HAL/LowLevelMemTracker.h"

#if UE_MEMORY_TRACE_ENABLED

#pragma push_macro("FVector")
#define FVector FVectorWorkaround
#import <Foundation/Foundation.h>
#undef FVector
#pragma pop_macro("FVector")

#include <Containers/Set.h>
#include <mach/mach.h>
#include <sys/types.h>
#include <sys/mman.h>

#include <dlfcn.h>

extern bool MemoryTrace_IsActive();

static float GAppleMemoryProbeInterval = 5.0f;
static FAutoConsoleVariableRef CAppleMemoryProbeInterval(
	TEXT("apple.memoryprobeinterval"),
	GAppleMemoryProbeInterval,
	TEXT("How often to sample virtual memory state in seconds, <=0.0f to disable. Only enabled if memory tracing is enabled.\n")
);

class FAppleMemoryProbe
{
public:
	~FAppleMemoryProbe()
	{
		Stop();
	}

	void Start()
	{
		bRunThread = true;

		ScanThread = MakeUnique<FThread>(TEXT("AppleMemoryProbe"), [this]()
		{
			while (bRunThread && (GAppleMemoryProbeInterval > 0.0f))
			{
				ScanMemory();
				FPlatformProcess::Sleep(GAppleMemoryProbeInterval);
			}
		}, 0, TPri_Lowest);
	}
	
	void Stop()
	{
		bRunThread = false;
		ScanThread->Join();
		ScanThread.Reset();
	}

private:
	struct FMemoryRegion
	{
		uint64 Begin; // could be 32 but 48 bit address space gives us 34 bits for page offset with 16kb pages
		uint64 End;

		FMemoryRegion(const uint64 InBegin, const uint64 InEnd)
		: Begin(InBegin), End(InEnd)
		{
		}

		FORCEINLINE bool Overlap(const FMemoryRegion& Other) const
		{
			return Begin <= Other.End && Other.Begin <= End;
		}
	};

	static constexpr uint64 kPtrUsableBits = 48; // 48 bit usable address space
	static constexpr uint64 kMinExpectedPageSize = 16 * 1024; // 16kb pages minimum
	static constexpr uint64 kPageMapsCount = 1024 * 1024; // 1024k pointers for 1 level indirect to cover whole address space (8MB)
	static constexpr uint64 kMemoryRegionsCount = 64 * 1024; // max count of memory regions in a process

	static constexpr uint64 kPtrUsableSize = 1ull << kPtrUsableBits; // 256 TB of usable address space
	static constexpr uint64 kUsablePages = kPtrUsableSize / kMinExpectedPageSize; // 17179869184 of usable pages
	static constexpr uint64 kPageMapSize = kUsablePages / kPageMapsCount; // 16384 pages per page map array (256 MB)
	static constexpr uint64 kPageMapSizeCompressed = kPageMapSize / 8; // 2kb per page map

	static_assert(kPageMapSizeCompressed == 2048);

	std::atomic<bool> bRunThread;
	TUniquePtr<FThread> ScanThread;

	// map all valid memory regions, sorted by address
	TArray<FMemoryRegion, TInlineAllocator<kMemoryRegionsCount>> MemoryRegions;

	// status of individual pages, 1 is in swap, 0 is in core or not present
	typedef TBitArray<TInlineAllocator<kPageMapSizeCompressed>> FPageMapArray;

	// pointers to arrays of kStatusVecCompressedSize size
	TUniquePtr<FPageMapArray> PageMaps[kPageMapsCount];

	// 16kb temp buffer to pass to mincore
	uint8 TempStatusVec[kPageMapSize];

	static const uint64 PageSize()
	{
		return vm_page_size;
	}

	static const FMemoryRegion PageMapIndexToMemoryRegion(const uint64 PageMapIndex)
	{
		return
		{
			PageMapIndex * kPageMapSize * PageSize(),
			(PageMapIndex + 1) * kPageMapSize * PageSize() - 1
		};
	}

	static const uint64 PageIndexToPtr(const uint64 PageMapIndex, const uint64 PageIndex)
	{
		return PageMapIndex * kPageMapSize * kMinExpectedPageSize + PageIndex * PageSize();
	}

	static constexpr bool ShouldScanRange(const uint32 VMTag)
	{
		// VM_MEMORY_APPLICATION_SPECIFIC_16 is 255 which is also -1, ignore it
		return VMTag >= VM_MEMORY_APPLICATION_SPECIFIC_1 && VMTag < VM_MEMORY_APPLICATION_SPECIFIC_16;
	}

	// partially or fully valid
	bool IsMemoryRegionValid(const FMemoryRegion& Region)
	{
		const int32 Index = Algo::UpperBound(MemoryRegions, Region, [](const FMemoryRegion& A, const FMemoryRegion& B) {return A.Begin < B.Begin;});
		return MemoryRegions[Index > 0 ? Index - 1 : 0].Overlap(Region);
	}

	void ScanMemory()
	{
		if (!MemoryTrace_IsActive())
		{
			return;
		}

		MemoryRegions.Empty();

		const mach_port_t Task = mach_task_self();
		vm_address_t NextAddr = 0;
		vm_size_t Size = 0;
		natural_t Depth = 0;
		vm_region_submap_info_data_64_t Info = {};
		mach_msg_type_number_t InfoCount = VM_REGION_SUBMAP_INFO_COUNT_64;
		uint64_t ProgramSize = 0;

		// Go over our memory map and gather all memory regions to track
		while (true)
		{
			if (vm_region_recurse_64(Task, &NextAddr, &Size, &Depth, (vm_region_info_t)&Info, &InfoCount) != KERN_SUCCESS)
			{
				NextAddr = -1;
				break;
			}

			const vm_address_t CurrAddr = NextAddr;
			NextAddr += Size;
			
			if(!Info.is_submap)
			{
				Dl_info dl_info{};
				dladdr(reinterpret_cast<void*>(CurrAddr), &dl_info);
				
				if(dl_info.dli_fname != nullptr && 
				   FAnsiStringView(dl_info.dli_fname).EndsWith(TCHAR_TO_ANSI(FPlatformProcess::ExecutableName())))
				{
					ProgramSize += Info.pages_resident * vm_page_size;
				}
			}
			
			if (!ShouldScanRange(Info.user_tag))
			{
				continue;
			}

			if (!MemoryRegions.IsEmpty())
			{
				FMemoryRegion& LastRegion = MemoryRegions.Last();
				if (LastRegion.End + 1 == CurrAddr)
				{
					LastRegion.End = NextAddr - 1; // coalesce with previous range
				}
				else
				{
					MemoryRegions.Emplace(CurrAddr, NextAddr - 1);
				}
			}
			else
			{
				MemoryRegions.Emplace(CurrAddr, NextAddr - 1);
			}
		}
		
	#if ENABLE_LOW_LEVEL_MEM_TRACKER
		FLowLevelMemTracker::Get().SetProgramSize(ProgramSize);
	#endif
		FGenericPlatformMemory::SetProgramSize(ProgramSize);
		
		struct
		{
			uint32 PageMaps;
			uint32 PageMapsFailed;
			uint32 IsPagedOut;
			uint32 IsPagedIn;
			uint32 IsFreedInSwap;
		} Stats = {};

		for (uint64 PageMapIndex = 0; PageMapIndex < kPageMapsCount; ++PageMapIndex)
		{
			TUniquePtr<FPageMapArray>& PageMap = PageMaps[PageMapIndex];

			const FMemoryRegion Region = PageMapIndexToMemoryRegion(PageMapIndex);
			if (IsMemoryRegionValid(Region))
			{
				// mincore on XNU doesn't fail on invalid mapings
				if (mincore((const void*)Region.Begin, kPageMapSize * PageSize(), (char*)TempStatusVec) == 0)
				{
					Stats.PageMaps++;
				}
				else
				{
					Stats.PageMapsFailed++;
					continue;
				}

				if (!PageMap.IsValid())
				{
					PageMap = MakeUnique<FPageMapArray>();
					PageMap->SetNum(kPageMapSize, false);
				}

				for (int32 PageIndex = 0; PageIndex < (int32)kPageMapSize; ++PageIndex)
				{
					FBitReference PageMapStatus = (*PageMap)[PageIndex];
					const bool bIsInCore = (TempStatusVec[PageIndex] & MINCORE_INCORE) != 0;
					const bool bIsPagedOut = (TempStatusVec[PageIndex] & MINCORE_PAGED_OUT) != 0;
					const bool bWasPagedOut = (bool)PageMapStatus;

					if (bWasPagedOut != bIsPagedOut)
					{
						PageMapStatus = bIsPagedOut;

						const uint64 Ptr = PageIndexToPtr(PageMapIndex, PageIndex);

						if (bIsPagedOut)
						{
							// Pages are stored in compressed form in swap.
							// It can be useful to know their compressed size to know how much physical memory they consume.
							// Currently there is no API to get this information from the kernel.
							//
							// Idea, we could use vm_copy to get the page binary data and compress it with lz4 ourselves,
							// to get an estimate of compressed size, but right now this will result in a page getting brought from swap,
							// as MADV_PAGEOUT is not available there is no easy way to page it out immediately.
							MemoryTrace_SwapOp(Ptr, EMemoryTraceSwapOperation::PageOut);
							Stats.IsPagedOut++;
						}
						else if (bIsInCore)
						{
							MemoryTrace_SwapOp(Ptr, EMemoryTraceSwapOperation::PageIn);
							Stats.IsPagedIn++;
						}
						else if (bWasPagedOut)
						{
							MemoryTrace_SwapOp(Ptr, EMemoryTraceSwapOperation::FreeInSwap);
							Stats.IsFreedInSwap++;
						}
					}
				}
			}
			else
			{
				if (PageMap.IsValid())
				{
					int32 PageIndex = INDEX_NONE;
					while ((PageIndex = PageMap->FindFrom(true, PageIndex == INDEX_NONE ? 0 : PageIndex + 1)) != INDEX_NONE)
					{
						const uint64 Ptr = PageIndexToPtr(PageMapIndex, PageIndex);
						MemoryTrace_SwapOp(Ptr, EMemoryTraceSwapOperation::FreeInSwap);
						Stats.IsFreedInSwap++;
					}

					PageMap.Reset();
				}
			}
		}
	}
};

void MemoryTrace_InitMemoryProbe()
{
	static TUniquePtr<FAppleMemoryProbe> MemoryProbe;

	// Delay the start of memory probe for a bit, because we're invoked right during malloc initialization.
	dispatch_after(dispatch_time(DISPATCH_TIME_NOW, 1 * NSEC_PER_SEC), dispatch_get_main_queue(), ^{
		MemoryProbe = MakeUnique<FAppleMemoryProbe>();
		MemoryProbe->Start();
	});
}

#endif // UE_MEMORY_TRACE_ENABLED
