// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"

#pragma push_macro("FVector")
#define FVector FVectorWorkaround
#import <Foundation/Foundation.h>
#undef FVector
#pragma pop_macro("FVector")

#include "HAL/PlatformMisc.h"
#include "HAL/PlatformProcess.h"
#include "Misc/AssertionMacros.h"
#include "ProfilingDebugging/ExternalProfiler.h"
#include "Features/IModularFeatures.h"
#include "Templates/UniquePtr.h"
#include "Apple/ApplePlatformDebugEvents.h"
#include "Misc/Paths.h"
#include "HAL/FileManager.h"

#if APPLE_PROFILING_ENABLED && UE_EXTERNAL_PROFILING_ENABLED

/**
 * Instruments implementation of FExternalProfiler
 */
class FInstrumentsExternalProfiler : public FExternalProfiler
{

public:

	/** Constructor */
	FInstrumentsExternalProfiler()
	{
		// Register as a modular feature
		IModularFeatures::Get().RegisterModularFeature( FExternalProfiler::GetFeatureName(), this );
	}


	/** Destructor */
	virtual ~FInstrumentsExternalProfiler()
	{
		IModularFeatures::Get().UnregisterModularFeature( FExternalProfiler::GetFeatureName(), this );
	}

	virtual void FrameSync() override
	{
		FApplePlatformDebugEvents::DebugSignPost(0, 0, 0, 0, 0);
	}

	/** Gets the name of this profiler as a string.  This is used to allow the user to select this profiler in a system configuration file or on the command-line */
	virtual const TCHAR* GetProfilerName() const override
	{
		return TEXT( "Instruments" );
	}


	/** Pauses profiling. */
	virtual void ProfilerPauseFunction() override
	{
		// Exit "instruments" command-line tool here
		if (Handle.IsValid())
		{
			FPlatformProcess::TerminateProc(Handle);
			FPlatformProcess::CloseProc(Handle);
		}
	}


	/** Resumes profiling. */
	virtual void ProfilerResumeFunction() override
	{
		// Start "instruments" command-line tool here
		if (!Handle.IsValid())
		{
			FString TemplatePath = FPaths::EngineDir() / TEXT("Instruments") / TEXT("UE4 System Trace.tracetemplate");
			if (!IFileManager::Get().FileExists(*TemplatePath))
			{
				TemplatePath = TEXT("Metal System Trace");
			}
			FString Params = FString::Printf(TEXT("-p %d -t \"%s\""), getpid(), *TemplatePath);
			Handle = FPlatformProcess::CreateProc( TEXT("/usr/bin/instruments"), *Params, false, false, false, nullptr, 0, nullptr, nullptr, nullptr);
		}
	}


	/**
	* Initializes profiler hooks. It is not valid to call pause/ Start on an uninitialized
	 * profiler and the profiler implementation is free to assert or have other undefined
	 * behavior.
	 *
	 * @return true if successful, false otherwise.
	 */
	bool Initialize()
	{
		return true;
	}


private:
	FProcHandle Handle;
};



namespace InstrumentsProfiler
{
	struct FAtModuleInit
	{
		FAtModuleInit()
		{
			static TUniquePtr<FInstrumentsExternalProfiler> ProfilerInstruments = MakeUnique<FInstrumentsExternalProfiler>();
			if( !ProfilerInstruments->Initialize() )
			{
				ProfilerInstruments.Reset();
			}
		}
	};

	static FAtModuleInit AtModuleInit;
}

#endif	// APPLE_PROFILING_ENABLED && UE_EXTERNAL_PROFILING_ENABLED
