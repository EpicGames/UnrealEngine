// Copyright Epic Games, Inc. All Rights Reserved.
#include "ProfilingDebugging/AssetMetadataTrace.h"

////////////////////////////////////////////////////////////////////////////////////////////////////
#if UE_TRACE_ASSET_METADATA_ENABLED

UE_TRACE_CHANNEL_DEFINE(AssetMetadataChannel,
	"Annotates other trace events with the asset/package they belong to (name, class, package, package-id->name mapping); "
	"consumed by TraceServices' MetadataProvider so other tools can group/filter events by asset.");
UE_TRACE_METADATA_EVENT_DEFINE(Asset);
UE_TRACE_METADATA_EVENT_DEFINE(PackageId)

UE_TRACE_EVENT_DEFINE(Package, PackageMapping);

#endif // UE_TRACE_ASSET_METADATA_ENABLED
