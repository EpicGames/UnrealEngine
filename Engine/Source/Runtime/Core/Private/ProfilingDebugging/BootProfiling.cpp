// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProfilingDebugging/BootProfiling.h"
#include "CoreGlobals.h"
#include "Misc/CoreDelegates.h"
#include "HAL/PlatformTime.h"

static double GEnginePreInitPreStartupScreenEndTime;
static double GEnginePreInitPostStartupScreenEndTime;
static double GEngineInitEndTime;
static bool GEngineInitializing = true;
static bool GCurrentlyInBackground = true;
static int GAppSuspendedCounter[2] = { 0,0 };
static int GAppActivatedCounter[2] = { 0,0 };
static double GBlockingDialogDuration = 0.0;

double FBootProfiling::GetBootDuration()
{
	return GEngineInitEndTime - GStartTime;
}

double FBootProfiling::GetPreInitPreStartupScreenDuration()
{
	return GEnginePreInitPreStartupScreenEndTime - GStartTime;
}

double FBootProfiling::GetPreInitPostStartupScreenDuration()
{
	return GEnginePreInitPostStartupScreenEndTime - GEnginePreInitPreStartupScreenEndTime;
}

double FBootProfiling::GetEngineInitDuration()
{
	return GEngineInitEndTime - GEnginePreInitPostStartupScreenEndTime;
}

void FBootProfiling::OnPreInitPreStartupScreenComplete()
{
	GEnginePreInitPreStartupScreenEndTime = FPlatformTime::Seconds();
}

void FBootProfiling::OnPreInitPostStartupScreenComplete()
{
	GEnginePreInitPostStartupScreenEndTime = FPlatformTime::Seconds();
}

void FBootProfiling::OnInitComplete()
{
	GEngineInitEndTime = FPlatformTime::Seconds();
	GEngineInitializing = false;
}

void FBootProfiling::InitCounters()
{
	FCoreDelegates::ApplicationWillEnterBackgroundDelegate.AddStatic([] 
		{ 
			++GAppSuspendedCounter[!GEngineInitializing]; 
			GCurrentlyInBackground = true;
		});
	FCoreDelegates::ApplicationHasEnteredForegroundDelegate.AddStatic([] 
		{ 
			if (!GCurrentlyInBackground)
			{
				// never saw a WillEnterBackground event, which happens on e.g. PS4, so we have to bump GAppSuspendedCounter
				++GAppSuspendedCounter[!GEngineInitializing];
			}
			++GAppActivatedCounter[!GEngineInitializing]; 
			GCurrentlyInBackground = false;
		});
}

static int GetPhaseCounter(FBootProfiling::ECounterPhase phase, const int counters[2])
{
	switch (phase)
	{
	case FBootProfiling::ECounterPhase::DuringEngineInit: return counters[0];
	case FBootProfiling::ECounterPhase::PostEngineInit: return counters[1];
	case FBootProfiling::ECounterPhase::AllPhases: return counters[0] + counters[1];
	default: return -1;
	}
}

int FBootProfiling::GetAppSuspendedCounter(ECounterPhase phase)
{
	return GetPhaseCounter(phase, GAppSuspendedCounter);
}

int FBootProfiling::GetAppActivatedCounter(ECounterPhase phase)
{
	return GetPhaseCounter(phase, GAppActivatedCounter);
}

void FBootProfiling::AddBlockingDialogDuration(double Seconds)
{
	GBlockingDialogDuration += Seconds;
}

double FBootProfiling::GetBlockingDialogDuration()
{
	return GBlockingDialogDuration;
}
