// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProfilingDebugging/CallstackTrace.h"
#include "CallstackTracePrivate.h"
#include "Containers/StaticArray.h"

#if UE_CALLSTACK_TRACE_ENABLED

// Platform implementations of back tracing
////////////////////////////////////////////////////////////////////////////////
void	CallstackTrace_CreateInternal(FMalloc*);
void	CallstackTrace_InitializeInternal();

////////////////////////////////////////////////////////////////////////////////
UE_TRACE_CHANNEL_DEFINE(CallstackChannel, "Allows allocations to be associated with callstacks. Requires Module channel to be enabled for symbol resolution to be possible.")
UE_TRACE_EVENT_DEFINE(Memory, CallstackSpec)
UE_TRACE_EVENT_DEFINE(Memory, CallstackSpecXORAndRLE)
UE_TRACE_EVENT_DEFINE(Memory, CallstackSpecDelta7bit)
UE_TRACE_EVENT_DEFINE(Memory, CallstackSpecDeltaVarInt)

uint32 GCallStackTracingTlsSlotIndex = FPlatformTLS::InvalidTlsSlot;

////////////////////////////////////////////////////////////////////////////////
void CallstackTrace_Create(class FMalloc* InMalloc)
{
	static auto InitOnce = [&]
	{
		CallstackTrace_CreateInternal(InMalloc);
		return true;
	}();
}

////////////////////////////////////////////////////////////////////////////////
void CallstackTrace_Initialize()
{
	GCallStackTracingTlsSlotIndex = FPlatformTLS::AllocTlsSlot();
	//NOTE: we don't bother cleaning up TLS, this is only closed during real shutdown

	static auto InitOnce = [&]
	{
		CallstackTrace_InitializeInternal();
		return true;
	}();
}

#endif //UE_CALLSTACK_TRACE_ENABLED
