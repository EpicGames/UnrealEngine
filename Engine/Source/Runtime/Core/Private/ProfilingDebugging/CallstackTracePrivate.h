// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ProfilingDebugging/CallstackTrace.h"
#include "CoreTypes.h"
#include "Experimental/Containers/GrowOnlyLockFreeHash.h"
#include "Trace/Trace.h"
#include "Trace/Trace.inl"

#if UE_CALLSTACK_TRACE_ENABLED

////////////////////////////////////////////////////////////////////////////////
#if !defined(UE_CALLSTACK_TRACE_RESERVE_MB)
	// Initial size of the known set of callstacks
	#if WITH_EDITOR
		#define UE_CALLSTACK_TRACE_RESERVE_MB 16 // ~1M callstacks
	#else
		#define UE_CALLSTACK_TRACE_RESERVE_MB 8 // ~500k callstacks
	#endif
#endif

#if !defined(UE_CALLSTACK_TRACE_RESERVE_GROWABLE)
	// If disabled the known set will not grow. New callstacks will not be
	// reported if the set is full
	#define UE_CALLSTACK_TRACE_RESERVE_GROWABLE 1
#endif

#if !defined(UE_CALLSTACK_TRACE_UNCOMPRESSED)
	// If enabled, callstack addresses aggregate without any compression
	#define UE_CALLSTACK_TRACE_UNCOMPRESSED 0
#endif

////////////////////////////////////////////////////////////////////////////////
UE_TRACE_CHANNEL_EXTERN(CallstackChannel, CORE_API)

UE_TRACE_EVENT_BEGIN_EXTERN(Memory, CallstackSpec, NoSync)
	UE_TRACE_EVENT_FIELD(uint32, CallstackId)
	UE_TRACE_EVENT_FIELD(uint64[], Frames)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN_EXTERN(Memory, CallstackSpecXORAndRLE, NoSync)
	UE_TRACE_EVENT_FIELD(uint32, CallstackId)
	UE_TRACE_EVENT_FIELD(uint8[], CompressedFrames)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN_EXTERN(Memory, CallstackSpecDelta7bit, NoSync)
	UE_TRACE_EVENT_FIELD(uint32, CallstackId)
	UE_TRACE_EVENT_FIELD(uint8[], CompressedFrames)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN_EXTERN(Memory, CallstackSpecDeltaVarInt, NoSync)
	UE_TRACE_EVENT_FIELD(uint32, CallstackId)
	UE_TRACE_EVENT_FIELD(uint8[], CompressedFrames)
UE_TRACE_EVENT_END()

////////////////////////////////////////////////////////////////////////////////

	class FCallstackTracer
	{
	public:
		struct FBacktraceEntry
		{
			uint64	Hash = 0;
			uint32	FrameCount = 0;
			uint64* Frames;
		};

		FCallstackTracer(FMalloc* InMalloc)
			: KnownSet(InMalloc)
			, CallstackIdCounter(1) // 0 is reserved for "unknown callstack"
		{
		}

		uint32 AddCallstack(const FBacktraceEntry& Entry)
		{
			bool bAlreadyAdded = false;

			// Our set implementation doesn't allow for zero entries (zero represents an empty element
			// in the hash table), so if we get one due to really bad luck in our 64-bit Id calculation,
			// treat it as a "1" instead, for purposes of tracking if we've seen that callstack.
			const uint64 Hash = FMath::Max(Entry.Hash, 1ull);
			uint32 Id; 
			KnownSet.Find(Hash, &Id, &bAlreadyAdded);
			if (!bAlreadyAdded)
			{
				Id = CallstackIdCounter.fetch_add(1, std::memory_order_relaxed);
				// On the first callstack reserve memory up front
				if (Id == 1)
				{
					KnownSet.Reserve(InitialReserveCount);
				}
#if !UE_CALLSTACK_TRACE_RESERVE_GROWABLE
				// If configured as not growable, start returning unknown id's when full.
				if (Id >= InitialReserveCount)
				{
					return 0;
				}
#endif //UE_CALLSTACK_TRACE_RESERVE_GROWABLE
				KnownSet.Emplace(Hash, Id);
#if UE_CALLSTACK_TRACE_UNCOMPRESSED
				UE_TRACE_LOG(Memory, CallstackSpec, CallstackChannel)
					<< CallstackSpec.CallstackId(Id)
					<< CallstackSpec.Frames(Entry.Frames, Entry.FrameCount);
#else
				uint8 Compressed[UE_CALLSTACK_TRACE_MAX_FRAMES * 9];
				const uint32 CompressedSize = FCallstackDeltaVarInt::Compress(
					MakeArrayView(Entry.Frames, Entry.FrameCount),
					Compressed
				);
				
				UE_TRACE_LOG(Memory, CallstackSpecDeltaVarInt, CallstackChannel)
					<< CallstackSpecDeltaVarInt.CallstackId(Id)
					<< CallstackSpecDeltaVarInt.CompressedFrames(Compressed, CompressedSize);
#endif //UE_CALLSTACK_TRACE_UNCOMPRESSED
			}

			return Id;
		}
	private:
		struct FEncounteredCallstackSetEntry
		{
			std::atomic_uint64_t Key;
			std::atomic_uint32_t Value;

			inline uint64 GetKey() const { return Key.load(std::memory_order_relaxed); }
			inline uint32 GetValue() const { return Value.load(std::memory_order_relaxed); }
			inline bool IsEmpty() const { return Key.load(std::memory_order_relaxed) == 0; }
			inline void SetKeyValue(uint64 InKey, uint32 InValue)
			{
				Value.store(InValue, std::memory_order_release);
				Key.store(InKey, std::memory_order_relaxed);
			}
			static inline uint32 KeyHash(uint64 Key) { return static_cast<uint32>(Key); }
			static inline void ClearEntries(FEncounteredCallstackSetEntry* Entries, int32 EntryCount)
			{
				memset(Entries, 0, EntryCount * sizeof(FEncounteredCallstackSetEntry));
			}
		};
		typedef TGrowOnlyLockFreeHash<FEncounteredCallstackSetEntry, uint64, uint32> FEncounteredCallstackSet;

		constexpr static uint32 InitialReserveBytes = UE_CALLSTACK_TRACE_RESERVE_MB * 1024 * 1024;
		constexpr static uint32 InitialReserveCount = InitialReserveBytes / sizeof(FEncounteredCallstackSetEntry);
		
		FEncounteredCallstackSet 	KnownSet;
		std::atomic_uint32_t		CallstackIdCounter;
	};

#endif //UE_CALLSTACK_TRACE_ENABLED
