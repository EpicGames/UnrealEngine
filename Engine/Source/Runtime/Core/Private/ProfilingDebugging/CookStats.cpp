// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProfilingDebugging/CookStats.h"

#include "Trace/Trace.inl"

#if ENABLE_COOK_STATS

CORE_API FCookStatsManager::FGatherCookStatsDelegate FCookStatsManager::CookStatsCallbacks;

CORE_API void FCookStatsManager::LogCookStats(AddStatFuncRef AddStat)
{
	CookStatsCallbacks.Broadcast(AddStat);
}

UE_TRACE_CHANNEL_DEFINE(CookChannel, "Package cook events: package names and IDs, asset class names, and per-package timing scopes for each cook processing stage. Enables the Unreal Cooking Insights (Packages) tab in Timing Insights profiler.")

UE_TRACE_EVENT_BEGIN(CookTrace, Package)
	UE_TRACE_EVENT_FIELD(uint64, Id)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, Name)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(CookTrace, PackageAssetClass)
	UE_TRACE_EVENT_FIELD(uint64, Id)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, ClassName)
UE_TRACE_EVENT_END()

void TracePackage(uint64 InId, const FStringView InName)
{
	UE_TRACE_LOG(CookTrace, Package, CookChannel)
		<< Package.Id(InId)
		<< Package.Name(InName.GetData(), uint16(InName.Len()))
		<< Package.Cycle(FPlatformTime::Cycles64());
}

void TracePackageAssetClass(uint64 InId, const FStringView InName)
{
	UE_TRACE_LOG(CookTrace, PackageAssetClass, CookChannel)
		<< PackageAssetClass.Id(InId)
		<< PackageAssetClass.ClassName(InName.GetData(), uint16(InName.Len()));
}

#if UE_INSIGHTS_COOK_PROFILER_VERSION == 2

UE_TRACE_EVENT_BEGIN(CookTrace, PackageStatBeginScope)
	UE_TRACE_EVENT_FIELD(uint64, Id)
	UE_TRACE_EVENT_FIELD(uint64, Time)
	UE_TRACE_EVENT_FIELD(uint8, StatType)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(CookTrace, PackageStatEndScope)
	UE_TRACE_EVENT_FIELD(uint64, Id)
	UE_TRACE_EVENT_FIELD(uint64, Time)
	UE_TRACE_EVENT_FIELD(uint8, StatType)
UE_TRACE_EVENT_END()

void TracePackageStatBeginScope(uint64 InId, uint64 Time, EPackageEventStatType StatType)
{
	UE_TRACE_LOG(CookTrace, PackageStatBeginScope, CookChannel)
		<< PackageStatBeginScope.Id(InId)
		<< PackageStatBeginScope.Time(Time)
		<< PackageStatBeginScope.StatType((uint8)StatType);
}

void TracePackageStatEndScope(uint64 InId, uint64 Time, EPackageEventStatType StatType)
{
	UE_TRACE_LOG(CookTrace, PackageStatEndScope, CookChannel)
		<< PackageStatEndScope.Id(InId)
		<< PackageStatEndScope.Time(Time)
		<< PackageStatEndScope.StatType((uint8)StatType);
}

#endif

#if UE_INSIGHTS_COOK_PROFILER_VERSION == 1

UE_TRACE_EVENT_BEGIN(CookTrace, PackageStat)
	UE_TRACE_EVENT_FIELD(uint64, Id)
	UE_TRACE_EVENT_FIELD(int64, Duration)
	UE_TRACE_EVENT_FIELD(uint8, StatType)
UE_TRACE_EVENT_END()

void TracePackageStat(uint64 InId, uint64 Duration, EPackageEventStatType StatType)
{
	UE_TRACE_LOG(CookTrace, PackageStat, CookChannel)
		<< PackageStat.Id(InId)
		<< PackageStat.Duration(Duration)
		<< PackageStat.StatType((uint8)StatType);
}

#endif

bool ShouldTracePackageInfo()
{
	return UE_TRACE_CHANNELEXPR_IS_ENABLED(CookChannel);
}

#endif
