// Copyright Epic Games, Inc. All Rights Reserved.
#include "ProfilingDebugging/CountersTrace.h"
#include "HAL/PlatformTime.h"
#include "Misc/Parse.h"
#include "Trace/Trace.inl"

#if COUNTERSTRACE_ENABLED

UE_TRACE_CHANNEL_DEFINE(CountersChannel, "Enables the Counters Trace API, allows tracing of values (float, integer) over time. \
Performance counters and statistics tracking. Data populates the Counters table in the Timing Insights profiler. \
It also enables the CSV Profiler Trace.")

UE_TRACE_EVENT_BEGIN(Counters, Spec, NoSync|Important)
	UE_TRACE_EVENT_FIELD(uint16, Id)
	UE_TRACE_EVENT_FIELD(uint8, Type)
	UE_TRACE_EVENT_FIELD(uint8, DisplayHint)
	UE_TRACE_EVENT_FIELD(UE::Trace::AnsiString, Name)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Counters, SetValueInt)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(int64, Value)
	UE_TRACE_EVENT_FIELD(uint16, CounterId)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Counters, SetValueFloat)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(double, Value)
	UE_TRACE_EVENT_FIELD(uint16, CounterId)
UE_TRACE_EVENT_END()

const TCHAR* FCountersTrace::AllocAndCopyCounterName(const TCHAR* InCounterName)
{
	int32 Len = FCString::Strlen(InCounterName);
	const TCHAR* CounterName = new TCHAR[Len + 1];
	FCString::Strncpy((TCHAR*)CounterName, InCounterName, Len + 1);
	return CounterName;
}

void FCountersTrace::FreeCounterName(const TCHAR* InCounterName)
{
	delete[] InCounterName;
}

uint16 FCountersTrace::OutputInitCounter(const TCHAR* CounterName, ETraceCounterType CounterType, ETraceCounterDisplayHint CounterDisplayHint)
{
	if (!UE_TRACE_CHANNELEXPR_IS_ENABLED(CountersChannel))
	{
		return 0;
	}

	static std::atomic<uint32> NextId { 0 };
	uint16 CounterId = uint16(++NextId);

	check(CounterName); // a trace counter object (TCounter<>) is used before it is constructed!?
	uint16 NameLen = uint16(FCString::Strlen(CounterName));

	UE_TRACE_LOG(Counters, Spec, CountersChannel, NameLen * sizeof(ANSICHAR))
		<< Spec.Id(CounterId)
		<< Spec.Type(uint8(CounterType))
		<< Spec.DisplayHint(uint8(CounterDisplayHint))
		<< Spec.Name(CounterName, NameLen);

	return CounterId;
}

void FCountersTrace::OutputSetValue(uint16 CounterId, int64 Value)
{
	if (!CounterId)
	{
		return;
	}

	UE_TRACE_LOG(Counters, SetValueInt, CountersChannel)
		<< SetValueInt.Cycle(FPlatformTime::Cycles64())
		<< SetValueInt.Value(Value)
		<< SetValueInt.CounterId(CounterId);
}

void FCountersTrace::OutputSetValue(uint16 CounterId, double Value)
{
	if (!CounterId)
	{
		return;
	}

	UE_TRACE_LOG(Counters, SetValueFloat, CountersChannel)
		<< SetValueFloat.Cycle(FPlatformTime::Cycles64())
		<< SetValueFloat.Value(Value)
		<< SetValueFloat.CounterId(CounterId);
}

#endif // COUNTERSTRACE_ENABLED
