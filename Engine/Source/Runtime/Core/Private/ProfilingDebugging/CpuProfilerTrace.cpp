// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProfilingDebugging/CpuProfilerTrace.h"

#if CPUPROFILERTRACE_ENABLED

#include "AutoRTFM.h"
#include "Containers/Map.h"
#include "HAL/LowLevelMemTracker.h"
#include "HAL/PlatformTime.h"
#include "HAL/PlatformTLS.h"
#include "HAL/TlsAutoCleanup.h"
#include "Misc/Crc.h"
#include "Misc/MemStack.h"
#include "Misc/Parse.h"
#include "ProfilingDebugging/MiscTrace.h"
#include "Trace/Trace.inl"
#include "UObject/NameTypes.h"

#if !defined(CPUPROFILERTRACE_FILE_AND_LINE_ENABLED)
	#define CPUPROFILERTRACE_FILE_AND_LINE_ENABLED 1
#endif

UE_TRACE_CHANNEL_DEFINE(CpuChannel, "CPU profiler scoping events and timing data. Captures hierarchical timing information for CPU-bound operations on all threads.")

UE_TRACE_EVENT_BEGIN(CpuProfiler, EventSpec, NoSync | Important)
	UE_TRACE_EVENT_FIELD(uint32, Id)
	UE_TRACE_EVENT_FIELD(UE::Trace::AnsiString, Name)
	#if CPUPROFILERTRACE_FILE_AND_LINE_ENABLED
	UE_TRACE_EVENT_FIELD(UE::Trace::AnsiString, File)
	UE_TRACE_EVENT_FIELD(uint32, Line)
	#endif
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(CpuProfiler, MetadataSpec, NoSync | Important) // added in UE 5.6
	UE_TRACE_EVENT_FIELD(uint32, Id)
	UE_TRACE_EVENT_FIELD(UE::Trace::AnsiString, Name)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, NameFormat)
	UE_TRACE_EVENT_FIELD(uint8[], FieldNames)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(CpuProfiler, Metadata, NoSync) // added in UE 5.6
	UE_TRACE_EVENT_FIELD(uint32, Id)
	UE_TRACE_EVENT_FIELD(uint32, SpecId)
	UE_TRACE_EVENT_FIELD(uint8[], Metadata)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(CpuProfiler, EventBatchV3, NoSync)
	UE_TRACE_EVENT_FIELD(uint8[], Data)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(CpuProfiler, EndThread, NoSync)
	UE_TRACE_EVENT_FIELD(uint64, Cycle) // added in UE 5.4
UE_TRACE_EVENT_END()

struct FCpuProfilerTraceInternal
{
	enum
	{
		MaxBufferSize = 256,
		MaxEncodedEventSize = 15, // 10 + 5
		FullBufferThreshold = MaxBufferSize - MaxEncodedEventSize,
	};

	template<typename CharType>
	struct FDynamicScopeNameMapKeyFuncs
	{
		typedef const CharType* KeyType;
		typedef const CharType* KeyInitType;
		typedef const TPairInitializer<const CharType*, uint32>& ElementInitType;

		enum { bAllowDuplicateKeys = false };

		static FORCEINLINE bool Matches(const CharType* A, const CharType* B)
		{
			return TCString<CharType>::Stricmp(A, B) == 0;
		}

		static FORCEINLINE uint32 GetKeyHash(const CharType* Key)
		{
			uint32 Hash = 0;
			for (const CharType* P = Key; *P; ++P)
			{
				Hash = ((Hash << 13) | (Hash >> 19)) ^ uint32(*P);
			}
			return Hash;
		}

		static FORCEINLINE KeyInitType GetSetKey(ElementInitType Element)
		{
			return Element.Key;
		}
	};

	enum class EThreadState
	{
		Uninitialized,
		Created,
		Destroyed
	};
	static thread_local EThreadState ThreadState;

	struct FThreadBuffer
		: public FTlsAutoCleanup
	{
		FThreadBuffer()
		{
			ThreadState = EThreadState::Created;
		}

		virtual ~FThreadBuffer()
		{
			if (BufferSize > 0)
			{
				FCpuProfilerTraceInternal::FlushThreadBuffer(this);
			}
			UE_TRACE_LOG(CpuProfiler, EndThread, CpuChannel)
				<< EndThread.Cycle(FPlatformTime::Cycles64());
			// Clear the thread buffer pointer. In the rare event of there being scopes in the destructors of other
			// FTLSAutoCleanup instances. In that case a new buffer is created for that event only. There is no way of
			// controlling the order of destruction for FTLSAutoCleanup types.
			ThreadBuffer = nullptr;
			ThreadState = EThreadState::Destroyed;
		}

		uint64 LastCycle = 0;
		uint16 BufferSize = 0;
		uint8 Buffer[MaxBufferSize];
		FMemStackBase DynamicScopeNamesMemory;
		TMap<const ANSICHAR*, uint32, FDefaultSetAllocator, FDynamicScopeNameMapKeyFuncs<ANSICHAR>> DynamicAnsiScopeNamesMap;
		TMap<const TCHAR*, uint32, FDefaultSetAllocator, FDynamicScopeNameMapKeyFuncs<TCHAR>> DynamicTCharScopeNamesMap;
		TMap<FNameEntryId, uint32> DynamicFNameScopeNamesMap;
	};

	uint32 static GetNextSpecId();
	uint32 static GetNextMetadataId();
	FORCENOINLINE static FThreadBuffer* CreateThreadBuffer();
	FORCENOINLINE static void FlushThreadBuffer(FThreadBuffer* ThreadBuffer);

	struct FSuspendScopes
	{
		uint32* TimerScopeDepth;
		uint32 SavedThreadDepth;
	};
	static thread_local TArray<FSuspendScopes, TInlineAllocator<3>> NestedTimerScopeDepths;
	static thread_local FThreadBuffer* ThreadBuffer;
	static thread_local uint32 ThreadDepth;
};

thread_local TArray<FCpuProfilerTraceInternal::FSuspendScopes, TInlineAllocator<3>> FCpuProfilerTraceInternal::NestedTimerScopeDepths;
thread_local FCpuProfilerTraceInternal::FThreadBuffer* FCpuProfilerTraceInternal::ThreadBuffer = nullptr;
thread_local uint32 FCpuProfilerTraceInternal::ThreadDepth = 0;
thread_local FCpuProfilerTraceInternal::EThreadState FCpuProfilerTraceInternal::ThreadState = FCpuProfilerTraceInternal::EThreadState::Uninitialized;

FCpuProfilerTraceInternal::FThreadBuffer* FCpuProfilerTraceInternal::CreateThreadBuffer()
{
	if (FCpuProfilerTraceInternal::ThreadState == FCpuProfilerTraceInternal::EThreadState::Destroyed)
	{
		return nullptr;
	}
	LLM_SCOPE_BYNAME(TEXT("Trace/CpuProfiler"));
	ThreadBuffer = new FThreadBuffer();
	ThreadBuffer->Register();
	return ThreadBuffer;
}

void FCpuProfilerTraceInternal::FlushThreadBuffer(FThreadBuffer* InThreadBuffer)
{
	UE_TRACE_LOG(CpuProfiler, EventBatchV3, true)
		<< EventBatchV3.Data(InThreadBuffer->Buffer, InThreadBuffer->BufferSize);
	InThreadBuffer->BufferSize = 0;
	InThreadBuffer->LastCycle = 0;
}

#define CPUPROFILERTRACE_OUTPUTBEGINEVENT_PROLOGUE() \
	++FCpuProfilerTraceInternal::ThreadDepth; \
	FCpuProfilerTraceInternal::FThreadBuffer* ThreadBuffer = FCpuProfilerTraceInternal::ThreadBuffer; \
	if (!ThreadBuffer) \
	{ \
		ThreadBuffer = FCpuProfilerTraceInternal::CreateThreadBuffer(); \
		if (!ThreadBuffer) \
		{ \
			return; \
		} \
	}

#define CPUPROFILERTRACE_OUTPUTBEGINEVENT_EPILOGUE(InSpecId) \
	uint64 Cycle = FPlatformTime::Cycles64(); \
	uint64 CycleDiff = Cycle - ThreadBuffer->LastCycle; \
	ThreadBuffer->LastCycle = Cycle; \
	uint8* BufferPtr = ThreadBuffer->Buffer + ThreadBuffer->BufferSize; \
	FTraceUtils::Encode7bit((CycleDiff << 2) | 1ull, BufferPtr); \
	FTraceUtils::Encode7bit(InSpecId, BufferPtr); \
	ThreadBuffer->BufferSize = (uint16)(BufferPtr - ThreadBuffer->Buffer); \
	if (ThreadBuffer->BufferSize >= FCpuProfilerTraceInternal::FullBufferThreshold) \
	{ \
		FCpuProfilerTraceInternal::FlushThreadBuffer(ThreadBuffer); \
	}

#define INSIGHTS_AUTORTFM_UNREACHABLE() \
	do { if (AutoRTFM::IsClosed()) { AutoRTFM::Unreachable("Unreachable transactional codepath in FCpuProfilerTrace"); } } while (0)

uint8 FCpuProfilerTrace::OnAbortKey = 42;

void FCpuProfilerTrace::GetOrCreateSpecId(uint32& InOutSpecId, const ANSICHAR* InEventName, const ANSICHAR* File, uint32 Line)
{
	AutoRTFM::Open([&]
	{
		// We only do relaxed here to avoid barrier cost as the worst case that can happen is multiple threads could each create an event type.
		// At some point the last thread in the race will set the output event and no more thread will try to create new ones from then on.
		// We don't care which event type wins as long as all threads eventually converge and stop creating new ones.
		if (FPlatformAtomics::AtomicRead_Relaxed((volatile int32*)&InOutSpecId) == 0)
		{
			FPlatformAtomics::AtomicStore_Relaxed((volatile int32*)&InOutSpecId, FCpuProfilerTrace::OutputEventType(InEventName, File, Line));
		}
	});
}

void FCpuProfilerTrace::GetOrCreateSpecId(uint32& InOutSpecId, const TCHAR* InEventName, const ANSICHAR* File, uint32 Line)
{
	AutoRTFM::Open([&]
	{
		// We only do relaxed here to avoid barrier cost as the worst case that can happen is multiple threads could each create an event type.
		// At some point the last thread in the race will set the output event and no more thread will try to create new ones from then on.
		// We don't care which event type wins as long as all threads eventually converge and stop creating new ones.
		if (FPlatformAtomics::AtomicRead_Relaxed((volatile int32*)&InOutSpecId) == 0)
		{
			FPlatformAtomics::AtomicStore_Relaxed((volatile int32*)&InOutSpecId, FCpuProfilerTrace::OutputEventType(InEventName, File, Line));
		}
	});
}

void FCpuProfilerTrace::GetOrCreateSpecId(uint32& InOutSpecId, FNameParam InEventName, const ANSICHAR* File, uint32 Line)
{
	AutoRTFM::Open([&]
	{
		// We only do relaxed here to avoid barrier cost as the worst case that can happen is multiple threads could each create an event type.
		// At some point the last thread in the race will set the output event and no more thread will try to create new ones from then on.
		// We don't care which event type wins as long as all threads eventually converge and stop creating new ones.
		if (FPlatformAtomics::AtomicRead_Relaxed((volatile int32*)&InOutSpecId) == 0)
		{
			FPlatformAtomics::AtomicStore_Relaxed((volatile int32*)&InOutSpecId, FCpuProfilerTrace::OutputEventType(InEventName, File, Line));
		}
	});
}

void FCpuProfilerTrace::OutputBeginEvent(uint32 SpecId)
{
	if (AutoRTFM::IsClosed())
	{
		AutoRTFM::Open([&]
		{
			OutputBeginEvent(SpecId);
		});

		AutoRTFM::PushOnAbortHandler(&FCpuProfilerTrace::OnAbortKey, []
		{
			OutputEndEvent();
		});

		return;
	}

	CPUPROFILERTRACE_OUTPUTBEGINEVENT_PROLOGUE();
	CPUPROFILERTRACE_OUTPUTBEGINEVENT_EPILOGUE(SpecId << 1 | 0);
}

void FCpuProfilerTrace::OutputBeginDynamicEvent(const ANSICHAR* Name, const ANSICHAR* File, uint32 Line)
{
	if (AutoRTFM::IsClosed())
	{
		AutoRTFM::Open([&]
		{
			OutputBeginDynamicEvent(Name, File, Line);
		});

		AutoRTFM::PushOnAbortHandler(&FCpuProfilerTrace::OnAbortKey, []
		{
			OutputEndEvent();
		});

		return;
	}

	CPUPROFILERTRACE_OUTPUTBEGINEVENT_PROLOGUE();
	uint32 SpecId = OutputDynamicEventType(Name, File, Line);
	CPUPROFILERTRACE_OUTPUTBEGINEVENT_EPILOGUE(SpecId << 1 | 0);
}

void FCpuProfilerTrace::OutputBeginDynamicEvent(const TCHAR* Name, const ANSICHAR* File, uint32 Line)
{
	if (AutoRTFM::IsClosed())
	{
		AutoRTFM::Open([&]
		{
			OutputBeginDynamicEvent(Name, File, Line);
		});

		AutoRTFM::PushOnAbortHandler(&FCpuProfilerTrace::OnAbortKey, []
		{
			OutputEndEvent();
		});

		return;
	}

	CPUPROFILERTRACE_OUTPUTBEGINEVENT_PROLOGUE();
	uint32 SpecId = OutputDynamicEventType(Name, File, Line);
	CPUPROFILERTRACE_OUTPUTBEGINEVENT_EPILOGUE(SpecId << 1 | 0);
}

void FCpuProfilerTrace::OutputBeginDynamicEvent(FNameParam Name, const ANSICHAR* File, uint32 Line)
{
	if (AutoRTFM::IsClosed())
	{
		AutoRTFM::Open([&]
		{
			OutputBeginDynamicEvent(Name, File, Line);
		});

		AutoRTFM::PushOnAbortHandler(&FCpuProfilerTrace::OnAbortKey, []
		{
			OutputEndEvent();
		});

		return;
	}

	CPUPROFILERTRACE_OUTPUTBEGINEVENT_PROLOGUE();
	uint32 SpecId = OutputDynamicEventType(Name, File, Line);
	CPUPROFILERTRACE_OUTPUTBEGINEVENT_EPILOGUE(SpecId << 1 | 0);
}

void FCpuProfilerTrace::OutputBeginDynamicEventWithId(FNameParam Id, const ANSICHAR* Name, const ANSICHAR* File, uint32 Line)
{
	if (AutoRTFM::IsClosed())
	{
		AutoRTFM::Open([&]
		{
			OutputBeginDynamicEventWithId(Id, Name, File, Line);
		});

		AutoRTFM::PushOnAbortHandler(&FCpuProfilerTrace::OnAbortKey, []
		{
			OutputEndEvent();
		});

		return;
	}

	CPUPROFILERTRACE_OUTPUTBEGINEVENT_PROLOGUE();
	uint32 SpecId = OutputDynamicEventTypeWithId(Id, Name, File, Line);
	CPUPROFILERTRACE_OUTPUTBEGINEVENT_EPILOGUE(SpecId << 1 | 0);
}

void FCpuProfilerTrace::OutputBeginDynamicEventWithId(FNameParam Id, const TCHAR* Name, const ANSICHAR* File, uint32 Line)
{
	if (AutoRTFM::IsClosed())
	{
		AutoRTFM::Open([&]
		{
			OutputBeginDynamicEventWithId(Id, Name, File, Line);
		});

		AutoRTFM::PushOnAbortHandler(&FCpuProfilerTrace::OnAbortKey, []
		{
			OutputEndEvent();
		});

		return;
	}

	CPUPROFILERTRACE_OUTPUTBEGINEVENT_PROLOGUE();
	uint32 SpecId = OutputDynamicEventTypeWithId(Id, Name, File, Line);
	CPUPROFILERTRACE_OUTPUTBEGINEVENT_EPILOGUE(SpecId << 1 | 0);
}

void FCpuProfilerTrace::OutputResumeEvent(uint64 SpecId, uint32& TimerScopeDepth)
{
	INSIGHTS_AUTORTFM_UNREACHABLE();

	FCpuProfilerTraceInternal::FThreadBuffer* ThreadBuffer = FCpuProfilerTraceInternal::ThreadBuffer;
	if (!ThreadBuffer)
	{
		ThreadBuffer = FCpuProfilerTraceInternal::CreateThreadBuffer();
		if (!ThreadBuffer)
		{
			return;
		}
	}

	FCpuProfilerTraceInternal::NestedTimerScopeDepths.Push({&TimerScopeDepth, FCpuProfilerTraceInternal::ThreadDepth});
	FCpuProfilerTraceInternal::ThreadDepth = FCpuProfilerTraceInternal::ThreadDepth + TimerScopeDepth;

	uint64 Cycle = FPlatformTime::Cycles64();
	uint64 CycleDiff = Cycle - ThreadBuffer->LastCycle;
	ThreadBuffer->LastCycle = Cycle;
	uint8* BufferPtr = ThreadBuffer->Buffer + ThreadBuffer->BufferSize;
	FTraceUtils::Encode7bit(CycleDiff << 2 | 3ull, BufferPtr);
	FTraceUtils::Encode7bit(SpecId, BufferPtr);
	FTraceUtils::Encode7bit(TimerScopeDepth, BufferPtr);
	ThreadBuffer->BufferSize = (uint16)(BufferPtr - ThreadBuffer->Buffer);

	if (ThreadBuffer->BufferSize >= FCpuProfilerTraceInternal::FullBufferThreshold)
	{
		FCpuProfilerTraceInternal::FlushThreadBuffer(ThreadBuffer);
	}
}

void FCpuProfilerTrace::OutputSuspendEvent()
{
	INSIGHTS_AUTORTFM_UNREACHABLE();

	FCpuProfilerTraceInternal::FThreadBuffer* ThreadBuffer = FCpuProfilerTraceInternal::ThreadBuffer;
	if (!ThreadBuffer)
	{
		ThreadBuffer = FCpuProfilerTraceInternal::CreateThreadBuffer();
		if (!ThreadBuffer)
		{
			return;
		}
	}

	auto [TimerScopeDepth, SavedThreadDepth] = FCpuProfilerTraceInternal::NestedTimerScopeDepths.Pop();
	*TimerScopeDepth = FCpuProfilerTraceInternal::ThreadDepth - SavedThreadDepth;
	FCpuProfilerTraceInternal::ThreadDepth = SavedThreadDepth;

	uint64 Cycle = FPlatformTime::Cycles64();
	uint64 CycleDiff = Cycle - ThreadBuffer->LastCycle;
	ThreadBuffer->LastCycle = Cycle;
	uint8* BufferPtr = ThreadBuffer->Buffer + ThreadBuffer->BufferSize;
	FTraceUtils::Encode7bit(CycleDiff << 2 | 2ull, BufferPtr);
	FTraceUtils::Encode7bit(uint64(*TimerScopeDepth), BufferPtr);
	ThreadBuffer->BufferSize = (uint16)(BufferPtr - ThreadBuffer->Buffer);

	if ((FCpuProfilerTraceInternal::ThreadDepth == 0) ||
		(ThreadBuffer->BufferSize >= FCpuProfilerTraceInternal::FullBufferThreshold))
	{
		FCpuProfilerTraceInternal::FlushThreadBuffer(ThreadBuffer);
	}
}

void FCpuProfilerTrace::OutputBeginEventWithMetadata(uint32 MetadataId)
{
	if (AutoRTFM::IsClosed())
	{
		AutoRTFM::Open([&]
			{
				OutputBeginEventWithMetadata(MetadataId);
			});

		AutoRTFM::PushOnAbortHandler(&FCpuProfilerTrace::OnAbortKey, []
			{
				OutputEndEventWithMetadata();
			});

		return;
	}

	CPUPROFILERTRACE_OUTPUTBEGINEVENT_PROLOGUE();
	CPUPROFILERTRACE_OUTPUTBEGINEVENT_EPILOGUE(MetadataId << 1 | 1);
}

#undef CPUPROFILERTRACE_OUTPUTBEGINEVENT_PROLOGUE
#undef CPUPROFILERTRACE_OUTPUTBEGINEVENT_EPILOGUE

void FCpuProfilerTrace::OutputEndEvent()
{
	if (AutoRTFM::IsClosed())
	{
		// For this to work correctly, the event *must* have been begun within the same transaction
		// as the end event was called. Otherwise the following could happen within a transaction:
		// - `OutputEndEvent` is called, which happens immediately.
		// - The transaction aborts.
		// - But we've unconditionally ended the event!

		AutoRTFM::Open([&]
		{
			OutputEndEvent();
		});

		AutoRTFM::PopOnAbortHandler(&FCpuProfilerTrace::OnAbortKey);

		return;
	}

	FCpuProfilerTraceInternal::FThreadBuffer* ThreadBuffer = FCpuProfilerTraceInternal::ThreadBuffer;
	if (!ThreadBuffer)
	{
		ThreadBuffer = FCpuProfilerTraceInternal::CreateThreadBuffer();
		if (!ThreadBuffer)
		{
			return;
		}
	}

	--FCpuProfilerTraceInternal::ThreadDepth;

	uint64 Cycle = FPlatformTime::Cycles64();
	uint64 CycleDiff = Cycle - ThreadBuffer->LastCycle;
	ThreadBuffer->LastCycle = Cycle;
	uint8* BufferPtr = ThreadBuffer->Buffer + ThreadBuffer->BufferSize;
	FTraceUtils::Encode7bit(CycleDiff << 2, BufferPtr);
	ThreadBuffer->BufferSize = (uint16)(BufferPtr - ThreadBuffer->Buffer);

	if ((FCpuProfilerTraceInternal::ThreadDepth == 0) ||
		(ThreadBuffer->BufferSize >= FCpuProfilerTraceInternal::FullBufferThreshold))
	{
		FCpuProfilerTraceInternal::FlushThreadBuffer(ThreadBuffer);
	}
}

uint32 FCpuProfilerTraceInternal::GetNextSpecId()
{
	static TAtomic<uint32> NextSpecId;
	return (NextSpecId++) + 1;
}

uint32 FCpuProfilerTraceInternal::GetNextMetadataId()
{
	static TAtomic<uint32> NextMetadataId;
	return (NextMetadataId++) + 1;
}

uint32 FCpuProfilerTrace::OutputEventType(const TCHAR* Name, const ANSICHAR* File, uint32 Line)
{
	if (AutoRTFM::IsClosed())
	{
		return AutoRTFM::Open([&]
		{
			return OutputEventType(Name, File, Line);
		});
	}

	uint32 SpecId = FCpuProfilerTraceInternal::GetNextSpecId();
	uint16 NameLen = uint16(FCString::Strlen(Name));
	uint32 DataSize = NameLen * sizeof(ANSICHAR); // EventSpec.Name is traced as UE::Trace::AnsiString
#if CPUPROFILERTRACE_FILE_AND_LINE_ENABLED
	uint16 FileLen = (File != nullptr) ? uint16(strlen(File)) : 0;
	DataSize += FileLen * sizeof(ANSICHAR);
#endif

	UE_TRACE_LOG(CpuProfiler, EventSpec, CpuChannel, DataSize)
		<< EventSpec.Id(SpecId)
		<< EventSpec.Name(Name, NameLen)
#if CPUPROFILERTRACE_FILE_AND_LINE_ENABLED
		<< EventSpec.File(File, FileLen)
		<< EventSpec.Line(Line)
#endif
	;
	return SpecId;
}

void FCpuProfilerTrace::OutputEventMetadataSpec(uint32 SpecId, const TCHAR* StaticName, const TCHAR* NameFormat, const TArray<uint8>& FieldNames)
{
	if (AutoRTFM::IsClosed())
	{
		return AutoRTFM::Open([&]
			{
				return OutputEventMetadataSpec(SpecId, StaticName, NameFormat, FieldNames);
			});
	}

	uint16 NameLen = uint16(FCString::Strlen(StaticName));
	uint32 DataSize = NameLen * sizeof(ANSICHAR); // EventSpec.Name is traced as UE::Trace::AnsiString
	uint16 NameFormatLen = uint16(FCString::Strlen(NameFormat));
	DataSize += NameFormatLen * sizeof(TCHAR);
	DataSize += FieldNames.Num() * sizeof(uint8);

	UE_TRACE_LOG(CpuProfiler, MetadataSpec, CpuChannel, DataSize)
		<< MetadataSpec.Id(SpecId)
		<< MetadataSpec.Name(StaticName, NameLen)
		<< MetadataSpec.NameFormat(NameFormat, NameFormatLen)
		<< MetadataSpec.FieldNames(FieldNames.GetData(), FieldNames.Num());
}

uint32 FCpuProfilerTrace::OutputMetadata(uint32 SpecId, const TArray<uint8>& InMetadata)
{
	if (AutoRTFM::IsClosed())
	{
		return AutoRTFM::Open([&]
			{
				return OutputMetadata(SpecId, InMetadata);
			});
	}

	if (!UE_TRACE_CHANNELEXPR_IS_ENABLED(CpuChannel))
	{
		return 0;
	}

	uint32 MetadataId = FCpuProfilerTraceInternal::GetNextMetadataId();

	UE_TRACE_LOG(CpuProfiler, Metadata, CpuChannel)
		<< Metadata.Id(MetadataId)
		<< Metadata.SpecId(SpecId)
		<< Metadata.Metadata(InMetadata.GetData(), InMetadata.Num());

	return MetadataId;
}

void FCpuProfilerTrace::OutputEndEventWithMetadata()
{
	OutputEndEvent();
}

uint32 FCpuProfilerTrace::OutputEventType(const ANSICHAR* Name, const ANSICHAR* File, uint32 Line)
{
	if (AutoRTFM::IsClosed())
	{
		return AutoRTFM::Open([&]
		{
			return OutputEventType(Name, File, Line);
		});
	}

	uint32 SpecId = FCpuProfilerTraceInternal::GetNextSpecId();
	uint16 NameLen = uint16(strlen(Name));
	uint32 DataSize = NameLen * sizeof(ANSICHAR);
#if CPUPROFILERTRACE_FILE_AND_LINE_ENABLED
	uint16 FileLen = (File != nullptr) ? uint16(strlen(File)) : 0;
	DataSize += FileLen * sizeof(ANSICHAR);
#endif

	UE_TRACE_LOG(CpuProfiler, EventSpec, CpuChannel, DataSize)
		<< EventSpec.Id(SpecId)
		<< EventSpec.Name(Name, NameLen)
#if CPUPROFILERTRACE_FILE_AND_LINE_ENABLED
		<< EventSpec.File(File, FileLen)
		<< EventSpec.Line(Line)
#endif
	;
	return SpecId;
}

uint32 FCpuProfilerTrace::OutputEventType(FNameParam Name, const ANSICHAR* File, uint32 Line)
{
	if (AutoRTFM::IsClosed())
	{
		return AutoRTFM::Open([&]
		{
			return OutputEventType(Name, File, Line);
		});
	}

	const FNameEntry* NameEntry = Name.GetDisplayNameEntry();
	if (NameEntry->IsWide())
	{
		WIDECHAR WideName[NAME_SIZE];
		NameEntry->GetWideName(WideName);
		return OutputEventType(WideName, File, Line);
	}
	else
	{
		ANSICHAR AnsiName[NAME_SIZE];
		NameEntry->GetAnsiName(AnsiName);
		return OutputEventType(AnsiName, File, Line);
	}
}

uint32 FCpuProfilerTrace::OutputDynamicEventType(const ANSICHAR* Name, const ANSICHAR* File, uint32 Line)
{
	if (AutoRTFM::IsClosed())
	{
		return AutoRTFM::Open([&]
		{
			return OutputDynamicEventType(Name, File, Line);
		});
	}

	FCpuProfilerTraceInternal::FThreadBuffer* ThreadBuffer = FCpuProfilerTraceInternal::ThreadBuffer;
	if (!ThreadBuffer)
	{
		ThreadBuffer = FCpuProfilerTraceInternal::CreateThreadBuffer();
		if (!ThreadBuffer)
		{
			return 0;
		}
	}

	uint32 SpecId = ThreadBuffer->DynamicAnsiScopeNamesMap.FindRef(Name);
	if (!SpecId)
	{
		LLM_SCOPE_BYNAME(TEXT("Trace/CpuProfiler"));
		int32 NameSize = strlen(Name) + 1;
		ANSICHAR* NameCopy = reinterpret_cast<ANSICHAR*>(ThreadBuffer->DynamicScopeNamesMemory.Alloc(NameSize, alignof(ANSICHAR)));
		FMemory::Memmove(NameCopy, Name, NameSize);
		SpecId = OutputEventType(NameCopy, File, Line);
		ThreadBuffer->DynamicAnsiScopeNamesMap.Add(NameCopy, SpecId);
	}
	return SpecId;
}

uint32 FCpuProfilerTrace::OutputDynamicEventType(const TCHAR* Name, const ANSICHAR* File, uint32 Line)
{
	if (AutoRTFM::IsClosed())
	{
		return AutoRTFM::Open([&]
		{
			return OutputDynamicEventType(Name, File, Line);
		});
	}

	FCpuProfilerTraceInternal::FThreadBuffer* ThreadBuffer = FCpuProfilerTraceInternal::ThreadBuffer;
	if (!ThreadBuffer)
	{
		ThreadBuffer = FCpuProfilerTraceInternal::CreateThreadBuffer();
		if (!ThreadBuffer)
		{
			return 0;
		}
	}

	uint32 SpecId = ThreadBuffer->DynamicTCharScopeNamesMap.FindRef(Name);
	if (!SpecId)
	{
		LLM_SCOPE_BYNAME(TEXT("Trace/CpuProfiler"));
		int32 NameSize = (FCString::Strlen(Name) + 1) * sizeof(TCHAR);
		TCHAR* NameCopy = reinterpret_cast<TCHAR*>(ThreadBuffer->DynamicScopeNamesMemory.Alloc(NameSize, alignof(TCHAR)));
		FMemory::Memmove(NameCopy, Name, NameSize);
		SpecId = OutputEventType(NameCopy, File, Line);
		ThreadBuffer->DynamicTCharScopeNamesMap.Add(NameCopy, SpecId);
	}
	return SpecId;
}

uint32 FCpuProfilerTrace::OutputDynamicEventType(FNameParam Name, const ANSICHAR* File, uint32 Line)
{
	if (AutoRTFM::IsClosed())
	{
		return AutoRTFM::Open([&]
		{
			return OutputDynamicEventType(Name, File, Line);
		});
	}

	FCpuProfilerTraceInternal::FThreadBuffer* ThreadBuffer = FCpuProfilerTraceInternal::ThreadBuffer;
	if (!ThreadBuffer)
	{
		ThreadBuffer = FCpuProfilerTraceInternal::CreateThreadBuffer();
		if (!ThreadBuffer)
		{
			return 0;
		}
	}

	uint32 SpecId = ThreadBuffer->DynamicFNameScopeNamesMap.FindRef(Name.GetComparisonIndex());
	if (!SpecId)
	{
		LLM_SCOPE_BYNAME(TEXT("Trace/CpuProfiler"));
		SpecId = OutputEventType(Name, File, Line);
		ThreadBuffer->DynamicFNameScopeNamesMap.Add(Name.GetComparisonIndex(), SpecId);
	}
	return SpecId;
}

uint32 FCpuProfilerTrace::OutputDynamicEventTypeWithId(FNameParam Id, const ANSICHAR* Name, const ANSICHAR* File, uint32 Line)
{
	if (AutoRTFM::IsClosed())
	{
		return AutoRTFM::Open([&]
		{
			return OutputDynamicEventTypeWithId(Id, Name, File, Line);
		});
	}

	FCpuProfilerTraceInternal::FThreadBuffer* ThreadBuffer = FCpuProfilerTraceInternal::ThreadBuffer;
	if (!ThreadBuffer)
	{
		ThreadBuffer = FCpuProfilerTraceInternal::CreateThreadBuffer();
		if (!ThreadBuffer)
		{
			return 0;
		}
	}

	uint32 SpecId = ThreadBuffer->DynamicFNameScopeNamesMap.FindRef(Id.GetComparisonIndex());
	if (!SpecId)
	{
		LLM_SCOPE_BYNAME(TEXT("Trace/CpuProfiler"));
		if (Name != nullptr)
		{
			SpecId = OutputEventType(Name, File, Line);
		}
		else
		{
			SpecId = OutputEventType(Id, File, Line);
		}
		ThreadBuffer->DynamicFNameScopeNamesMap.Add(Id.GetComparisonIndex(), SpecId);
	}
	return SpecId;
}

uint32 FCpuProfilerTrace::OutputDynamicEventTypeWithId(FNameParam Id, const TCHAR* Name, const ANSICHAR* File, uint32 Line)
{
	if (AutoRTFM::IsClosed())
	{
		return AutoRTFM::Open([&]
		{
			return OutputDynamicEventTypeWithId(Id, Name, File, Line);
		});
	}

	FCpuProfilerTraceInternal::FThreadBuffer* ThreadBuffer = FCpuProfilerTraceInternal::ThreadBuffer;
	if (!ThreadBuffer)
	{
		ThreadBuffer = FCpuProfilerTraceInternal::CreateThreadBuffer();
		if (!ThreadBuffer)
		{
			return 0;
		}
	}

	uint32 SpecId = ThreadBuffer->DynamicFNameScopeNamesMap.FindRef(Id.GetComparisonIndex());
	if (!SpecId)
	{
		LLM_SCOPE_BYNAME(TEXT("Trace/CpuProfiler"));
		if (Name != nullptr)
		{
			SpecId = OutputEventType(Name, File, Line);
		}
		else
		{
			SpecId = OutputEventType(Id, File, Line);
		}
		ThreadBuffer->DynamicFNameScopeNamesMap.Add(Id.GetComparisonIndex(), SpecId);
	}
	return SpecId;
}

void FCpuProfilerTrace::FlushThreadBuffer()
{
	if (AutoRTFM::IsClosed())
	{
		AutoRTFM::Open([&]
		{
			FlushThreadBuffer();
		});

		return;
	}

	FCpuProfilerTraceInternal::FThreadBuffer* ThreadBuffer = FCpuProfilerTraceInternal::ThreadBuffer;
	if (ThreadBuffer && ThreadBuffer->BufferSize > 0)
	{
		FCpuProfilerTraceInternal::FlushThreadBuffer(ThreadBuffer);
	}
}

#endif // CPUPROFILERTRACE_ENABLED
