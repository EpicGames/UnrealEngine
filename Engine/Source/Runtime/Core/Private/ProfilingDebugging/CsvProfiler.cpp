// Copyright Epic Games, Inc. All Rights Reserved.

/**
*
* A lightweight multi-threaded profiler with very low instrumentation overhead. Suitable for Test or even final Shipping builds
* Results are accumulated per-frame and emitted in CSV format
*/

#include "ProfilingDebugging/CsvProfiler.h"
#include "Async/Fundamental/Scheduler.h"
#include "Async/ParallelFor.h"
#include "Async/TaskGraphInterfaces.h"
#include "CoreGlobals.h"
#include "HAL/RunnableThread.h"
#include "HAL/ThreadManager.h"
#include "HAL/ThreadHeartBeat.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformProcess.h"
#include "HAL/PlatformTime.h"
#include "HAL/IConsoleManager.h"
#include "Misc/Paths.h"
#include "Misc/CommandLine.h"
#include "Misc/ScopeLock.h"
#include "Misc/CoreMisc.h"
#include "Containers/Map.h"
#include "Misc/CoreDelegates.h"
#include "Misc/App.h"
#include "HAL/Runnable.h"
#include "Misc/EngineVersion.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "Stats/Stats.h"
#include "Stats/ThreadIdleStats.h"
#include "HAL/LowLevelMemTracker.h"
#include "Misc/Compression.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/Fork.h"
#include "Misc/WildcardString.h"
#include "Modules/ModuleManager.h"
#include "HAL/PlatformMemoryHelpers.h"
#include "HAL/PlatformMisc.h"

#include <limits>

#if CSV_PROFILER

#define CSV_PROFILER_INLINE FORCEINLINE

#ifndef CSV_PROFILER_SUPPORT_NAMED_EVENTS
// This doesn't actually enable named events. Use -csvNamedEvents or -csvNamedEventsTiming to enable for exclusive or normal timing stats respectively
#define CSV_PROFILER_SUPPORT_NAMED_EVENTS ENABLE_NAMED_EVENTS
#endif

#ifndef CSV_PROFILER_ALLOW_SENSITIVE_BUILTIN_METADATA
// Disable sensitive metadata like LoginId and Commandline outside of shipping builds
#define CSV_PROFILER_ALLOW_SENSITIVE_BUILTIN_METADATA !UE_BUILD_SHIPPING
#endif

#define REPAIR_MARKER_STACKS 1

// Note: Enabling this will break a lot of reporting but it may be useful for debugging. If disabled (default), the thread name is omitted from custom stat columns
// This results in multiple columns with the same name (since we have 1 column per stat per thread), but these are automatically combined by the CSV tools
#define CSV_DEBUG_CUSTOM_STATS_INCLUDE_THREAD_NAME 0

// By default we hide per thread stats when aggregation is enabled (since the point is to reduce the number). Enabling this shows both
#define CSV_DEBUG_EMIT_SEPARATE_THREAD_STATS_WHEN_TASK_AGGREGATION_ENABLED 0


// Global CSV category (no prefix)
FCsvCategory GGlobalCsvCategory(TEXT("GLOBAL"), true, true);

// Basic high level perf category
CSV_DEFINE_CATEGORY_MODULE(CORE_API, Basic, true);
CSV_DEFINE_CATEGORY_MODULE(CORE_API, Exclusive, true);
CSV_DEFINE_CATEGORY_MODULE(CORE_API, FileIO, true);

// Other categories
CSV_DEFINE_CATEGORY(CsvProfiler, true);
CSV_DEFINE_CATEGORY(CsvBench, true);

#if CSV_PROFILER_ALLOW_DEBUG_FEATURES
	CSV_DEFINE_CATEGORY(CsvTest, true);
	static bool GCsvTestingGT = false;
	static bool GCsvTestingRT = false;
	static bool GCsvTestCategoryOnly = false;
	void CSVTest();
#endif
static bool GAllCategoriesStartDisabled = false;

#define LIST_VALIDATION (DO_CHECK && 0)

DEFINE_LOG_CATEGORY_STATIC(LogCsvProfiler, Log, All);

TAutoConsoleVariable<int32> CVarCsvBlockOnCaptureEnd(
	TEXT("csv.BlockOnCaptureEnd"), 
	1,
	TEXT("When 1, blocks the game thread until the CSV file has been written completely when the capture is ended.\r\n")
	TEXT("When 0, the game thread is not blocked whilst the file is written."),
	ECVF_Default
);

TAutoConsoleVariable<bool> CVarCsvAggregateTaskWorkerStats(
	TEXT("csv.AggregateTaskWorkerStats"),
	true,
	TEXT("If enabled, stats recorded on task worker threads are aggregated instead of outputting a single stat per thread.\r\n")
	TEXT("This reduces CSV bloat when there are large numbers of worker threads and makes stat data more intelligible"),
	ECVF_Default
);

TAutoConsoleVariable<int32> CVarCsvContinuousWrites(
	TEXT("csv.ContinuousWrites"),
	1,
	TEXT("When 1, completed CSV rows are converted to CSV format strings and appended to the write buffer whilst the capture is in progress.\r\n")
	TEXT("When 0, CSV rows are accumulated in memory as binary data, and only converted to strings and flushed to disk at the end of the capture."),
	ECVF_Default
);

TAutoConsoleVariable<int32> CVarCsvForceExit(
	TEXT("csv.ForceExit"),
	0,
	TEXT("If 1, do a forced exit when if exitOnCompletion is enabled"),
	ECVF_Default
);

TAutoConsoleVariable<bool> CVarCsvBenchmark(
	TEXT("csv.Benchmark"),
#if UE_BUILD_SHIPPING
	false,
#else
	true,
#endif
	TEXT("If emabled, do a quick benchmark test on the frame before the CSV profiler starts"),
	ECVF_Default
);

TAutoConsoleVariable<int32> CVarCsvBenchmarkIterationCount(
	TEXT("csv.Benchmark.IterationCount"),
	10000,
	TEXT("Number of iterations of each CsvBenchmark test"),
	ECVF_Default
);


TAutoConsoleVariable<int32> CVarCsvTargetFrameRateOverride(
	TEXT("csv.TargetFrameRateOverride"),
	0,
	TEXT("If 0, Defaults to calculating the target frame rate using rhi.SyncInterval and Max refresh rate."),
	ECVF_Default
);

TAutoConsoleVariable<bool> CVarCsvUseLegacyFrameTime(
	TEXT("csv.UseLegacyFrameTime"),
	false,
	TEXT("If true uses the delta between EndFrame() calls to compute the frame time.")
	TEXT("If false, computes the frame time after the idle at the start of the frame (when using a frame rate limit)."),
	ECVF_Default
);

#if UE_BUILD_SHIPPING
TAutoConsoleVariable<int32> CVarCsvShippingContinuousWrites(
	TEXT("csv.Shipping.ContinuousWrites"),
	-1,
	TEXT("Only applies in shipping buids. If != -1, overrides csv.ContinousWrites."),
	ECVF_Default
);

TAutoConsoleVariable<int32> CVarCsvShippingMaxFileSizeMB(
	TEXT("csv.Shipping.MaxFileSizeMB"),
	-1,
	TEXT("Only applies in shipping buids. If != -1, overrides csv.MaxFileSizeMB."),
	ECVF_Default
);
#endif

TAutoConsoleVariable<int32> CVarCsvMaxFileSizeMB(
	TEXT("csv.MaxFileSizeMB"),
	0,
	TEXT("Approximate maximum file size (in megabytes) for CSV profiler files.")
	TEXT("The profiler will stop writing values to the output file after this size is exceeded.")
	TEXT("Only supported when ContinuousWrites cvar is > 0.")
	TEXT("If <= 0, size is not limited."),
	ECVF_Default
);

TAutoConsoleVariable<int32> CVarCsvCompressionMode(
	TEXT("csv.CompressionMode"),
	-1,
	TEXT("Controls whether CSV files are compressed when written out.\r\n")
	TEXT(" -1 = (Default) Use compression if the code which started the capture opted for it.\r\n")
	TEXT("  0 = Force disable compression. All files will be written as uncompressed .csv files.\r\n")
	TEXT("  1 = Force enable compression. All files will be written as compressed .csv.gz files."),
	ECVF_Default
);

TAutoConsoleVariable<int32> CVarCsvStatCounts(
	TEXT("csv.statCounts"),
	0,
	TEXT("If 1, outputs count stats"),
	ECVF_Default
);

TAutoConsoleVariable<int32> CVarCsvWriteBufferSize(
	TEXT("csv.WriteBufferSize"),
	128 * 1024, // 128 KB
	TEXT("When non-zero, defines the size of the write buffer to use whilst writing the CSV file.\r\n")
	TEXT("A non-zero value is required for GZip compressed output."),
	ECVF_Default
);

TAutoConsoleVariable<int32> CVarCsvStreamFramesToBuffer(
	TEXT("csv.FramesToBuffer"),
	128,
	TEXT("Defines the minimum amount of frames to keep in memory before flushing them."),
	ECVF_Default
);

TAutoConsoleVariable<int32> CVarCsvPauseProcessingThread(
	TEXT("csv.PauseProcessingThread"),
	0,
	TEXT("Debug only - When 1, blocks the processing thread to simulate starvation"),
	ECVF_Default
);

TAutoConsoleVariable<float> CVarCsvProcessingThreadTimeBetweenUpdates(
	TEXT("csv.ProcessingThread.TimeBetweenUpdates"),
	50.0f,
	TEXT("Specifies the minimum time between CSV processing thread updates.\r\n")
	TEXT("Note: This is the time between the start of updates. If processing takes longer than this, the next update will commence immediately."),
	ECVF_Default
);

TAutoConsoleVariable<float> CVarCsvProcessingThreadGtStallMaxUpdateLatencyMs(
	TEXT("csv.ProcessingThread.GtStallMaxUpdateLatencyMs"),
	0.0,
	TEXT("How far behind the processing thread can get (ms since last update) before we stall the GT. Detection is handled on the GT. Set to 0 to disable"),
	ECVF_Default
);

TAutoConsoleVariable<float> CVarCsvProcessingThreadGtStallUpdateTimeThresholdMs(
	TEXT("csv.ProcessingThread.GtStallUpdateTimeThresholdMs"),
	1000.0f,
	TEXT("Specifies the max time a processing thread update can take before we consider stalling the game thread. Used in combination with csv.ProcessingThread.GtStallUpdateEscalationThreshold\n")
	TEXT("Set to 0 to disable stalling"),
	ECVF_Default
);

TAutoConsoleVariable<int32> CVarCsvProcessingThreadGtStallUpdateEscalationThreshold(
	TEXT("csv.ProcessingThread.GtStallUpdateEscalationThreshold"),
	3,
	TEXT("Number of progressively slower updates before we stall the GT. Updates faster than UpdateTimeThresholdMs are ignored"),
	ECVF_Default
);


TAutoConsoleVariable<int32> CVarMaxPerThreadStatDataSlackKB(
	TEXT("csv.MaxPerThreadStatDataSlackKB"),
	64,
	TEXT("Max amount of per thread slack data to allow during a capture.\r\n")
	TEXT("Higher values result in better performance due to fewer allocations but higher memory overhead"),
	ECVF_Default
);

TAutoConsoleVariable<bool> CVarNumberedFNamesStatsAreFatal(
	TEXT("csv.NumberedFNamesStatsAreFatal"),
	false,
	TEXT("Fatal error if numbered fname stats are encountered (ignored in shipping)"),
	ECVF_Default
);

TAutoConsoleVariable<int32> CVarCsvStatNameValidation(
	TEXT("csv.StatNameValidation"),
	2,
	TEXT("If 0, does nothing\r\n")
	TEXT("If 1, warns if there are invalid characters\r\n")
	TEXT("If 2, warns and sanitizes stat names with invalid characters\r\n")
	TEXT("If 3, warns and removes stats with invalid characters"),
	ECVF_Default
);

// Thread priority cvar (settable at runtime)
// We declare these explicitly rather than just casting the cvar in case the enum changes in future
const EThreadPriority GCsvThreadPriorities[] =
{
	EThreadPriority::TPri_Lowest,
	EThreadPriority::TPri_BelowNormal,
	EThreadPriority::TPri_SlightlyBelowNormal,
	EThreadPriority::TPri_Normal,
	EThreadPriority::TPri_AboveNormal,
	EThreadPriority::TPri_Highest
};

const TCHAR* GCsvThreadPriorityNames[] =
{
	TEXT("TPri_Lowest"),
	TEXT("TPri_BelowNormal"),
	TEXT("TPri_SlightlyBelowNormal"),
	TEXT("TPri_Normal"),
	TEXT("TPri_AboveNormal"),
	TEXT("TPri_Highest")
};

static int32 GCsvThreadPriorityIndex = 0; 
FAutoConsoleVariableRef CVarCsvThreadPriority(
	TEXT("csv.ProcessingThread.Priority"),
	GCsvThreadPriorityIndex,
	TEXT("Thread priority of the CSV profiler thread: 0=Lowest, 1=BelowNormal, 2=SlightlyBelowNormal, 3=Normal, 4=AboveNormal, 5=Highest\n")
	TEXT("This is switchable at runtime"),
	ECVF_Default);

TAutoConsoleVariable<int32> CVarCsvTrackMemoryUse(
	TEXT("csv.TrackMemoryUse"),
	0,
	TEXT("if 1, tracks memory use via the CsvProfiler/MemUsedLastUpdateMB stat. This can take 0.5ms on the processing thread so is disabled by default"),
	ECVF_Default
);


static bool GCsvUseProcessingThread = true;
static int32 GCsvRepeatCount = 0;
static int32 GCsvRepeatFrameCount = 0;
static bool GCsvStatCounts = false;
static FString* GStartOnEvent = nullptr;
static FString* GStopOnEvent = nullptr;
static uint32 GCsvProcessingThreadId = 0;
static bool GGameThreadIsCsvProcessingThread = true;

static uint32 GCsvProfilerFrameNumber = 0;

static bool GCsvTrackWaitsOnAllThreads = false;
static bool GCsvTrackWaitsOnGameThread = true;
static bool GCsvTrackWaitsOnRenderThread = true;
static FAutoConsoleVariableRef CVarTrackWaitsAllThreads(TEXT("csv.trackWaitsAllThreads"), GCsvTrackWaitsOnAllThreads, TEXT("Determines whether to track waits on all threads. Note that this incurs a lot of overhead"), ECVF_Default);
static FAutoConsoleVariableRef CVarTrackWaitsGT(TEXT("csv.trackWaitsGT"), GCsvTrackWaitsOnGameThread, TEXT("Determines whether to track game thread waits. Note that this incurs overhead"), ECVF_Default);
static FAutoConsoleVariableRef CVarTrackWaitsRT(TEXT("csv.trackWaitsRT"), GCsvTrackWaitsOnRenderThread, TEXT("Determines whether to track render thread waits. Note that this incurs overhead"), ECVF_Default);

// Set via config [CsvProfiler]:bEventTimestamps. Defaults to true
// If enabled, CSV events will include a timestamp, encoded with the following suffix : ##<time in seconds>
static bool GCsvEventTimestamps = true;

//
// Categories
//
static const uint32 CSV_MAX_CATEGORY_COUNT = 2048;
static TSAN_ATOMIC(bool) GCsvCategoriesEnabled[CSV_MAX_CATEGORY_COUNT];

static TSAN_ATOMIC(bool) GCsvProfilerIsCapturing = false;
static bool GCsvProfilerIsCapturingRT = false; // Renderthread version of the above

static bool GCsvProfilerIsWritingFile = false;
static FString GCsvFileName = FString();
static bool GCsvExitOnCompletion = false;

static thread_local bool GCsvThreadLocalWaitsEnabled = false;

static uint64 GCaptureStartTimestamp = 0;

static bool GCsvProfilerInitialized = false;

static std::atomic<bool> GDelayProcessingThreadStart = false;

//
// Forward declarations
//
struct FCsvStatSeries;
struct FCsvAggregateStatSeries;
struct FCsvProcessThreadDataStats;
class FCsvThreadGroupStatProcessor;

// A unique ID for a CSV stat, either ansi or FName
union FCsvUniqueStatID
{
public:
	FCsvUniqueStatID(const FCsvUniqueStatID& Src)
	{
		Hash = Src.Hash;
	}
	FCsvUniqueStatID(uint64 InStatIDRaw, int32 InCategoryIndex, bool bInIsFName, bool bInIsCountStat = false)
	{
		check(InCategoryIndex < CSV_MAX_CATEGORY_COUNT);
		Fields.IsFName = bInIsFName ? 1 : 0;
		Fields.FNameOrIndex = InStatIDRaw;
		Fields.CategoryIndex = InCategoryIndex;
		Fields.IsCountStat = bInIsCountStat ? 1 : 0;
	}
	FCsvUniqueStatID(const FName& Name, int32 InCategoryIndex)
	{
		check(InCategoryIndex < CSV_MAX_CATEGORY_COUNT);
		Fields.FNameOrIndex = Name.ToUnstableInt();
		Fields.CategoryIndex = InCategoryIndex;
		Fields.IsFName = 1;
		Fields.IsCountStat = 0;
	}
	struct
	{
		uint64 IsFName : 1;
		uint64 IsCountStat : 1;
		uint64 CategoryIndex : 11;
		uint64 FNameOrIndex : 51;
	} Fields;
	uint64 Hash;
};


// Persistent custom stats
struct FCsvPersistentCustomStats
{
	void RecordStats()
	{
		UE::TScopeLock Lock(Cs);
		for (FCsvPersistentCustomStatBase* BaseStat : Stats)
		{
			switch(BaseStat->GetStatType())
			{
				case ECsvPersistentCustomStatType::Float:
				{
					RecordStat<float>(BaseStat);
					break;
				}
				case ECsvPersistentCustomStatType::Int:
				{
					RecordStat<int32>(BaseStat);
					break;
				}
			}
		}
	}


	template<class T>
	TCsvPersistentCustomStat<T>* GetOrCreatePersistentCustomStat(FName Name, int32 CategoryIndex, bool bResetEachFrame)
	{
		LLM_SCOPE(ELLMTag::CsvProfiler);
		UE::TScopeLock Lock(Cs);
		FCsvUniqueStatID Id(Name, CategoryIndex);
		FCsvPersistentCustomStatBase** FindStat = StatLookup.Find(Id.Hash);
		if (FindStat)
		{
			if (TCsvPersistentCustomStat<T>::GetClassStatType() == (*FindStat)->GetStatType())
			{
				return static_cast<TCsvPersistentCustomStat<T>*>(*FindStat);
			}
			UE_LOGF(LogCsvProfiler, Fatal, "Error: Custom stat %ls was already registered with a different type", *Name.ToString());
		}
		// This will leak, and that's ok. These stats are intended to persist for the lifetime of the program
		TCsvPersistentCustomStat<T>* NewStat = new TCsvPersistentCustomStat<T>(Name, CategoryIndex, bResetEachFrame);
		StatLookup.Add(Id.Hash, NewStat);
		Stats.Add(NewStat);
		return NewStat;
	}

	template <class T>
	void RecordStat(FCsvPersistentCustomStatBase* BaseStat)
	{
		TCsvPersistentCustomStat<T>* Stat = static_cast<TCsvPersistentCustomStat<T>*>(BaseStat);
		FCsvProfiler::RecordCustomStat(Stat->Name, Stat->CategoryIndex, Stat->GetValue(), ECsvCustomStatOp::Set);
		if (Stat->bResetEachFrame)
		{
			Stat->Set(0);
		}
	}

	FCriticalSection Cs;
	TMap<uint64, FCsvPersistentCustomStatBase*> StatLookup;
	TArray<FCsvPersistentCustomStatBase*> Stats;
};
static FCsvPersistentCustomStats GCsvPersistentCustomStats;



#if CSV_PROFILER_SUPPORT_NAMED_EVENTS
bool GCsvProfilerNamedEventsExclusive = false;
bool GCsvProfilerNamedEventsTiming = false;

static FAutoConsoleVariableRef CVarNamedEventsExclusive(TEXT("csv.NamedEventsExclusive"), GCsvProfilerNamedEventsExclusive, TEXT("Determines whether to emit named events for exclusive stats"), ECVF_Default);
static FAutoConsoleVariableRef CVarNamedEventsTiming(TEXT("csv.NamedEventsTiming"), GCsvProfilerNamedEventsTiming, TEXT("Determines whether to emit named events for non-exclusive timing stats"), ECVF_Default);

void CsvBeginNamedEvent(FColor Color, const char* NamedEventName)
{
#if CPUPROFILERTRACE_ENABLED
	if (UE_TRACE_CHANNELEXPR_IS_ENABLED(CpuChannel))
	{
		FCpuProfilerTrace::OutputBeginDynamicEvent(NamedEventName, __FILE__, __LINE__);
	}
	else
#endif
	{
#if PLATFORM_IMPLEMENTS_BeginNamedEventStatic
		FPlatformMisc::BeginNamedEventStatic(Color, NamedEventName);
#else
		FPlatformMisc::BeginNamedEvent(Color, NamedEventName);
#endif
	}
}

void CsvBeginNamedEvent(FColor Color, const FName& StatName)
{
#if CPUPROFILERTRACE_ENABLED
	if (UE_TRACE_CHANNELEXPR_IS_ENABLED(CpuChannel))
	{
		FCpuProfilerTrace::OutputBeginDynamicEvent(StatName, __FILE__, __LINE__);
	}
	else
#endif
	{
#if PLATFORM_IMPLEMENTS_BeginNamedEventStatic
		FPlatformMisc::BeginNamedEventStatic(Color, *StatName.ToString());
#else
		FPlatformMisc::BeginNamedEvent(Color, *StatName.ToString());
#endif
	}
}

void CsvEndNamedEvent()
{
#if CPUPROFILERTRACE_ENABLED
	if (UE_TRACE_CHANNELEXPR_IS_ENABLED(CpuChannel))
	{
		FCpuProfilerTrace::OutputEndEvent();
	}
	else
#endif
	{
		FPlatformMisc::EndNamedEvent();
	}
}
#endif //CSV_PROFILER_SUPPORT_NAMED_EVENTS

static TMap<uint32, TArray<FString>>* GCsvFrameExecCmds = NULL;
static TMap<uint32, TArray<FString>>* GCsvFrameModuloExecCmds = NULL;

struct FEventExecCmds
{
	alignas(PLATFORM_CACHE_LINE_SIZE) volatile int32 bIsActive;
	FString EventWildcard;
	TArray<FString> Cmds;
};
static TArray<FEventExecCmds>* GCsvEventExecCmds = NULL;

bool IsContinuousWriteEnabled(bool bGameThread)
{
#if CSV_PROFILER_ALLOW_DEBUG_FEATURES
	static bool bContinuousWritesDisabled = FParse::Param(FCommandLine::Get(), TEXT("csvContinuousWritesDisabled"));
	if (bContinuousWritesDisabled)
	{
		return false;
	}
#endif
	int32 CVarValue = -1;
#if UE_BUILD_SHIPPING
	CVarValue = bGameThread ? CVarCsvShippingContinuousWrites.GetValueOnGameThread() : CVarCsvShippingContinuousWrites.GetValueOnAnyThread();
	if (CVarValue == -1)
#endif
	{
		CVarValue = bGameThread ? CVarCsvContinuousWrites.GetValueOnGameThread() : CVarCsvContinuousWrites.GetValueOnAnyThread();
	}
	return CVarValue > 0;
} 

int32 GetMaxFileSizeMB(bool bGameThread)
{
	int32 CVarValue = -1;
#if UE_BUILD_SHIPPING
	CVarValue = bGameThread ? CVarCsvShippingMaxFileSizeMB.GetValueOnGameThread() : CVarCsvShippingMaxFileSizeMB.GetValueOnAnyThread();
	if (CVarValue == -1)
#endif
	{
		CVarValue = bGameThread ? CVarCsvMaxFileSizeMB.GetValueOnGameThread() : CVarCsvMaxFileSizeMB.GetValueOnAnyThread();
	}
	return CVarValue;
}

static CSV_PROFILER_INLINE void ValidateFName(const FName& StatName)
{
	ensureMsgf(StatName.GetNumber() == 0, TEXT("Numbered FName stats (suffixed _<number>) are not supported. Stat name: '%s'"), *StatName.ToString());
#if UE_FNAME_OUTLINE_NUMBER && !UE_BUILD_SHIPPING
	if (StatName.GetNumber() > 0 && CVarNumberedFNamesStatsAreFatal.GetValueOnAnyThread())
	{
		UE_LOGF(LogCsvProfiler, Fatal, "Numbered FName stats (suffixed _<number>) are not supported. Stat name: '%ls'. Disable csv.NumberedFNamesStatsAreFatal to make this non-fatal", *StatName.ToString());
	}
#endif
}

#if CSV_PROFILER_ALLOW_DEBUG_FEATURES
class FCsvABTest
{
public:
	FCsvABTest()
		: StatFrameOffset(0)
		, SwitchDuration(7)
		, bPrevCapturing(false)
		, bFastCVarSet(false)
	{}

	void AddCVarABData(const FString& CVarName, int32 Count)
	{
		Count = CVarValues.Num() - Count;
		IConsoleVariable* ConsoleVariable = IConsoleManager::Get().FindConsoleVariable(*CVarName);

		if (Count > 0 && ConsoleVariable != nullptr)
		{
			CVarABDataArray.Add({ CVarName, *CVarName, ConsoleVariable, ConsoleVariable->GetString(), Count, FLT_MAX });
		}
		else if (ConsoleVariable == nullptr)
		{
			UE_LOGF(LogCsvProfiler, Log, "Skipping CVar %ls - Not found", *CVarName);
		}
		else if (Count == 0)
		{
			UE_LOGF(LogCsvProfiler, Log, "Skipping CVar %ls - No value specified", *CVarName);
		}
	}

	void IterateABTestArguments(const FString& ABTestString)
	{
		int32 FindIndex;
		if (!ABTestString.FindChar(TEXT('='), FindIndex))
		{
			return;
		}

		int32 Count = CVarValues.Num();

		FString CVarName = ABTestString.Mid(0, FindIndex);
		FString ValueStr = ABTestString.Mid(FindIndex + 1);
		while (true)
		{
			int32 CommaIndex;
			bool bComma = ValueStr.FindChar(TEXT(','), CommaIndex);
			int32 SemiColonIndex;
			bool bSemiColon = ValueStr.FindChar(TEXT(';'), SemiColonIndex);

			if (bComma)
			{
				if (!bSemiColon || (bSemiColon && CommaIndex<SemiColonIndex))
				{
					FString Val = ValueStr.Mid(0, CommaIndex);
					CVarValues.Add(FCString::Atof(*Val));
					ValueStr.MidInline(CommaIndex + 1, MAX_int32, EAllowShrinking::No);
					continue;
				}
			}

			if (bSemiColon)
			{
				if (SemiColonIndex==0)
				{
					AddCVarABData(CVarName, Count);
					IterateABTestArguments(ValueStr.Mid(SemiColonIndex + 1));
					break;
				}
				else
				{
					FString Val = ValueStr.Mid(0, SemiColonIndex);
					CVarValues.Add(FCString::Atof(*Val));
					ValueStr.MidInline(SemiColonIndex, MAX_int32, EAllowShrinking::No);
					continue;
				}
			}

			CVarValues.Add(FCString::Atof(*ValueStr));
			AddCVarABData(CVarName, Count);
			break;
		}

	}

	void InitFromCommandline()
	{
		FString ABTestString;
		if (FParse::Value(FCommandLine::Get(), TEXT("csvABTest="), ABTestString, false))
		{
			IterateABTestArguments(ABTestString);

			if (CVarABDataArray.Num() > 0)
			{
				UE_LOGF(LogCsvProfiler, Log, "Initialized CSV Profiler A/B test"); 
				
				int32 CVarValuesIndex = 0;
				for (int32 Index = 0; Index < CVarABDataArray.Num(); ++Index)
				{
					const FCVarABData& CVarABData = CVarABDataArray[Index];

					UE_LOGF(LogCsvProfiler, Log, "  CVar %ls [Original value: %ls] AB Test with values:", *CVarABData.CVarName, *CVarABData.OriginalValue);
					for (int32 i = 0; i < CVarABData.Count; ++i)
					{
						UE_LOGF(LogCsvProfiler, Log, "    [%d] : %.2f", i, CVarValues[CVarValuesIndex + i]);
					}

					CVarValuesIndex += CVarABData.Count;
				}

				FParse::Value(FCommandLine::Get(), TEXT("csvABTestStatFrameOffset="), StatFrameOffset);
				FParse::Value(FCommandLine::Get(), TEXT("csvABTestSwitchDuration="), SwitchDuration);
				bFastCVarSet = FParse::Param(FCommandLine::Get(), TEXT("csvABTestFastCVarSet"));
				UE_LOGF(LogCsvProfiler, Log, "Stat Offset: %d frames", StatFrameOffset);
				UE_LOGF(LogCsvProfiler, Log, "Switch Duration : %d frames", SwitchDuration);
				UE_LOGF(LogCsvProfiler, Log, "Fast cvar set: %ls", bFastCVarSet ? TEXT("Enabled") : TEXT("Disabled"));

			}
			else
			{
				UE_LOGF(LogCsvProfiler, Log, "CSV Profiler A/B has not initialized");
			}
		}
	}

	void BeginFrameUpdate(int32 FrameNumber, bool bCapturing)
	{
		if (CVarABDataArray.Num() == 0)
		{
			return;
		}
		
		if (bCapturing)
		{
			int32 CVarValuesIndex = 0;
			for (int32 Index = 0; Index < CVarABDataArray.Num(); ++Index)
			{
				FCVarABData& CVarABData = CVarABDataArray[Index];

				int32 ValueIndex = (FrameNumber / SwitchDuration) % CVarABData.Count;
				int32 StatValueIndex = ((FrameNumber - StatFrameOffset) / SwitchDuration) % CVarABData.Count;
				
				ValueIndex += CVarValuesIndex;
				StatValueIndex += CVarValuesIndex;
				CVarValuesIndex += CVarABData.Count;

				{
					float Value = CVarValues[ValueIndex];
					if (Value != CVarABData.PreviousValue)
					{
						EConsoleVariableFlags CVarFlags = ECVF_SetByCode;
						if (bFastCVarSet)
						{
							CVarFlags = EConsoleVariableFlags(CVarFlags | ECVF_Set_NoSinkCall_Unsafe);
						}
						CVarABData.ConsoleVariable->Set(*FString::Printf(TEXT("%f"), Value), CVarFlags);

						CVarABData.PreviousValue = Value;
					}
				}

				FCsvProfiler::RecordCustomStat(CVarABData.CVarStatFName, CSV_CATEGORY_INDEX_GLOBAL, CVarValues[StatValueIndex], ECsvCustomStatOp::Set);
			}
		}
		else if (bPrevCapturing == true)
		{
			// Restore cvar to old value
			// TODO: Set Setby flag to the original value
			for (int32 Index = 0; Index < CVarABDataArray.Num(); ++Index)
			{
				CVarABDataArray[Index].ConsoleVariable->Set(*CVarABDataArray[Index].OriginalValue);

				UE_LOGF(LogCsvProfiler, Log, "CSV Profiler A/B test - setting %ls=%ls", *CVarABDataArray[Index].CVarName, *CVarABDataArray[Index].OriginalValue);
			}

		}
		bPrevCapturing = bCapturing;
	}

private:
	struct FCVarABData
	{
		FString CVarName;
		FName CVarStatFName;
		IConsoleVariable* ConsoleVariable;
		FString OriginalValue;
		int32 Count;
		float PreviousValue;
	};
	
	TArray<FCVarABData> CVarABDataArray;
	TArray<float> CVarValues;

	int32 StatFrameOffset;
	int32 SwitchDuration;
	bool bPrevCapturing;
	bool bFastCVarSet;
};
static FCsvABTest GCsvABTest;

#endif // CSV_PROFILER_ALLOW_DEBUG_FEATURES

struct FCsvBenchmarkResult
{
	double TimingStatTimeNs = 0.0;
	double CustomStatTimeNs = 0.0;
};

class FCsvBenchmark
{
	struct FTimer
	{
		FTimer()
		{
			StartCycles = FPlatformTime::Cycles64();
		}
		double GetTimeElapsed()
		{
			uint64 CyclesElapsed = FPlatformTime::Cycles64() - StartCycles;
			return FPlatformTime::ToSeconds64(CyclesElapsed);
		}
		uint64 StartCycles = 0;
	};
	bool bIsRunning = false;

public:
	FCsvBenchmarkResult Run(int32 IterationCount)
	{
		CSV_SCOPED_TIMING_STAT_EXCLUSIVE(CsvBench);
		bIsRunning = true;
		FCsvBenchmarkResult Result;
		FTimer MainTimer;

		FTimer TimingStatTimer;
		for (int i=0; i<IterationCount; i++)
		{
			CSV_SCOPED_TIMING_STAT(CsvBench, TimingStat1);
			CSV_SCOPED_TIMING_STAT(CsvBench, TimingStat2);
			CSV_SCOPED_TIMING_STAT(CsvBench, TimingStat3);
			CSV_SCOPED_TIMING_STAT(CsvBench, TimingStat4);
		}
		Result.TimingStatTimeNs = TimingStatTimer.GetTimeElapsed() * 1000000000.0 / double(IterationCount * 4);

		FTimer CustomStatTimer;
		for (int i = 0; i < IterationCount; i++)
		{
			CSV_CUSTOM_STAT(CsvBench, CustomStat, 1.0f, ECsvCustomStatOp::Accumulate);
			CSV_CUSTOM_STAT(CsvBench, CustomStat, 1.0f, ECsvCustomStatOp::Accumulate);
			CSV_CUSTOM_STAT(CsvBench, CustomStat, 1.0f, ECsvCustomStatOp::Accumulate);
			CSV_CUSTOM_STAT(CsvBench, CustomStat, 1.0f, ECsvCustomStatOp::Accumulate);
		}
		Result.CustomStatTimeNs = CustomStatTimer.GetTimeElapsed() * 1000000000.0 / double(IterationCount * 4);

		double BenchTotalTimeMs = MainTimer.GetTimeElapsed() * 1000.0;
		UE_LOGF(LogCsvProfiler, Log, "Csv profiler benchmark completed (%d iterations). Time taken: %.2fms", IterationCount, BenchTotalTimeMs );
		UE_LOGF(LogCsvProfiler, Log, "Timing stat ns: %.2fns", Result.TimingStatTimeNs);
		UE_LOGF(LogCsvProfiler, Log, "Custom stat ns: %.2fns", Result.CustomStatTimeNs);

		CSV_METADATA(TEXT("CsvBench_TimingStat_ns"), *LexToString(Result.TimingStatTimeNs));
		CSV_METADATA(TEXT("CsvBench_CustomStat_ns"), *LexToString(Result.CustomStatTimeNs));
		CSV_METADATA(TEXT("CsvBench_Duration_ms"), *LexToString(BenchTotalTimeMs));
		bIsRunning = false;
		return Result;
	}

	bool IsRunning() const
	{
		return bIsRunning;
	}
};
static FCsvBenchmark GCsvBenchmark;


class FCsvCategoryData
{
public:
	static FCsvCategoryData* Get()
	{
		if (!Instance)
		{
			Instance = new FCsvCategoryData;
			FMemory::Memzero(GCsvCategoriesEnabled, sizeof(GCsvCategoriesEnabled));
		}
		return Instance;
	}

	FString GetCategoryNameByIndex(int32 Index) const
	{
		UE::TScopeLock Lock(CS);
		return CategoryNames[Index];
	}

	int32 GetCategoryCount() const
	{
		return CategoryNames.Num();
	}

	int32 GetCategoryIndex(const FString& CategoryName) const
	{
		UE::TScopeLock Lock(CS);
		const int32* CategoryIndex = CategoryNameToIndex.Find(CategoryName.ToLower());
		if (CategoryIndex)
		{
			return *CategoryIndex;
		}
		return -1;
	}

	void UpdateCategoryFromConfig(int32 CategoryIndex)
	{
		if (GAllCategoriesStartDisabled)
		{
			return;
		}
		for (FString const& DisabledCategory : CategoriesDisabledInConfig)
		{
			if (FWildcardString::IsMatch(*DisabledCategory, *CategoryNames[CategoryIndex]))
			{
				UE_LOGF(LogCsvProfiler, Log, "Config disabled category %ls", *CategoryNames[CategoryIndex]);
				GCsvCategoriesEnabled[CategoryIndex] = false;
			}
		}
		for (FString const& EnabledCategory : CategoriesEnabledInConfig)
		{
			if (FWildcardString::IsMatch(*EnabledCategory, *CategoryNames[CategoryIndex]))
			{
				UE_LOGF(LogCsvProfiler, Log, "Config enabled category %ls", *CategoryNames[CategoryIndex]);
				GCsvCategoriesEnabled[CategoryIndex] = true;
			}
		}
	}

	void UpdateCategoriesFromConfig()
	{
		TArray<FString> NewCategoriesDisabledInConfig;
		TArray<FString> NewCategoriesEnabledInConfig;

		GConfig->GetArray(TEXT("CsvProfiler"), TEXT("EnabledCategories"), NewCategoriesEnabledInConfig, GEngineIni);
		GConfig->GetArray(TEXT("CsvProfiler"), TEXT("DisabledCategories"), NewCategoriesDisabledInConfig, GEngineIni);

		// Check if the config changed. This prevents us from resetting categories unnecessarily at runtime (which could be confusing if debug category toggle commands are in play)
		if (NewCategoriesEnabledInConfig != CategoriesEnabledInConfig || NewCategoriesDisabledInConfig != CategoriesDisabledInConfig )
		{
			CategoriesEnabledInConfig = NewCategoriesEnabledInConfig;
			CategoriesDisabledInConfig = NewCategoriesDisabledInConfig;
			for (int i = 0; i < GetCategoryCount(); ++i)
			{
				UpdateCategoryFromConfig(i);
			}
		}
	}

	int32 RegisterCategory(const FString& CategoryName, bool bEnableByDefault, bool bIsGlobal)
	{
		int32 Index = -1;
		// During a hot-reload, we attempt to re-register categories and/or statics
		// that not have been loaded/init'd during a hot-reload, can result in crashing.
		// Thus when doing a reload, bail. 
		if (IsReloadActive())
		{
			return Index;
		}
		if (GAllCategoriesStartDisabled)
		{
			bEnableByDefault = false;
		}
		UE::TScopeLock Lock(CS);
		{
			Index = GetCategoryIndex(CategoryName);
			if (ensureMsgf(Index == -1, TEXT("CSV stat category already declared: %s. Note: Categories are not case sensitive"), *CategoryName))
			{
				if (bIsGlobal)
				{
					Index = 0;
				}
				else
				{
					Index = CategoryNames.Num();
					CategoryNames.AddDefaulted();
				}
				check(Index < CSV_MAX_CATEGORY_COUNT);
				if (Index < CSV_MAX_CATEGORY_COUNT)
				{
					GCsvCategoriesEnabled[Index] = bEnableByDefault;
					CategoryNames[Index] = CategoryName;
					CategoryNameToIndex.Add(CategoryName.ToLower(), Index);
					UpdateCategoryFromConfig(Index);
				}
				TRACE_CSV_PROFILER_REGISTER_CATEGORY(Index, *CategoryName);
			}
		}
		return Index;
	}


private:
	FCsvCategoryData()
	{
		// Category 0 is reserved for the global category
		CategoryNames.AddDefaulted(1);
	}

	mutable FCriticalSection CS;
	TMap<FString, int32> CategoryNameToIndex;
	TArray<FString> CategoryNames;
	TArray<FString> CategoriesDisabledInConfig;
	TArray<FString> CategoriesEnabledInConfig;

	static FCsvCategoryData* Instance;
};
FCsvCategoryData* FCsvCategoryData::Instance = nullptr;


int32 FCsvProfiler::GetCategoryIndex(const FString& CategoryName)
{
	return FCsvCategoryData::Get()->GetCategoryIndex(CategoryName);
}

int32 FCsvProfiler::RegisterCategory(const FString& CategoryName, bool bEnableByDefault, bool bIsGlobal)
{
	return FCsvCategoryData::Get()->RegisterCategory(CategoryName, bEnableByDefault, bIsGlobal);
}

void FCsvProfiler::GetFrameExecCommands(TArray<FString>& OutFrameCommands)
{
	check(IsInGameThread());
	OutFrameCommands.Empty();
	if (!GCsvProfilerIsCapturing)
	{
		return;
	}
	if (GCsvFrameExecCmds)
	{
		TArray<FString>* FrameCommands = GCsvFrameExecCmds->Find(CaptureFrameNumber);
		if (FrameCommands)
		{
			OutFrameCommands = *FrameCommands;
		}
	}
	if (GCsvFrameModuloExecCmds)
	{
		for (auto It(GCsvFrameModuloExecCmds->CreateConstIterator()); It; ++It)
		{
			uint32 FrameModulo = It.Key();
			if ((CaptureFrameNumber % FrameModulo) == 0)
			{
				OutFrameCommands.Append(It.Value());
			}
		}
	}
	if (GCsvEventExecCmds)
	{
		for (FEventExecCmds& EventPair : *GCsvEventExecCmds)
		{
			if (FPlatformAtomics::InterlockedExchange(&EventPair.bIsActive, (int32)0) > 0)
			{
				OutFrameCommands.Append(EventPair.Cmds);
			}
		}
	}
}


bool IsInCsvProcessingThread()
{
	uint32 ProcessingThreadId = GGameThreadIsCsvProcessingThread ? GGameThreadId : GCsvProcessingThreadId;
	return FPlatformTLS::GetCurrentThreadId() == ProcessingThreadId;
}

static void HandleCSVProfileCommand(const TArray<FString>& Args)
{
	if (Args.Num() < 1)
	{
		return;
	}

	FString Param = Args[0];

	if (Param == TEXT("START"))
	{
		FCsvProfiler::Get()->BeginCapture(-1, FString(), GCsvFileName);
	}
	else if (Param == TEXT("STOP"))
	{
		FCsvProfiler::Get()->EndCapture();
	}
	else if (FParse::Value(*Param, TEXT("STARTFILE="), GCsvFileName))
	{
	}
	else if (Param == TEXT("EXITONCOMPLETION"))
	{
		GCsvExitOnCompletion = true;
	}
	else
	{
		int32 CaptureFrames = 0;
		if (FParse::Value(*Param, TEXT("FRAMES="), CaptureFrames))
		{
			FCsvProfiler::Get()->BeginCapture(CaptureFrames, FString(), GCsvFileName);
		}
		int32 RepeatCount = 0;
		if (FParse::Value(*Param, TEXT("REPEAT="), RepeatCount))
		{
			GCsvRepeatCount = RepeatCount;
		}
	}
}

static void CsvProfilerBeginFrame()
{
	FCsvProfiler::Get()->BeginFrame();
}

static void CsvProfilerEndFrame()
{
	FCsvProfiler::Get()->EndFrame();
}

static void CsvProfilerBeginFrameRT()
{
	FCsvProfiler::Get()->BeginFrameRT();
}

static void CsvProfilerEndFrameRT()
{
	FCsvProfiler::Get()->EndFrameRT();
}

static void CsvProfilerReadConfig()
{
	check(GConfig);
	GConfig->GetBool(TEXT("CsvProfiler"), TEXT("bEventTimestamps"), GCsvEventTimestamps, GEngineIni);
	FCsvCategoryData::Get()->UpdateCategoriesFromConfig();
}


static FAutoConsoleCommand HandleCSVProfileCmd(
	TEXT("CsvProfile"),
	TEXT("Starts or stops Csv Profiles"),
	FConsoleCommandWithArgsDelegate::CreateStatic(&HandleCSVProfileCommand)
);

static void HandleCSVCategoryCommand(const TArray<FString>& Args, UWorld* World, FOutputDevice& OutputDevice)
{
	if ((Args.Num() >= 1) && (Args.Num() <= 2))
	{
		const FCsvProfiler* CsvProfiler = FCsvProfiler::Get();
		const FString& Category = Args[0];
		const int32 CategoryIndex = CsvProfiler->GetCategoryIndex(Category);
		if (CategoryIndex < 0)
		{
			OutputDevice.Logf(ELogVerbosity::Error, TEXT("CsvProfiler: category '%s' does not exist."), *Category);
			return;
		}

		bool bEnabled = true;
		bool bIsOperationValid = true;
		if (Args.Num() == 2)
		{
			const FString& Operation = Args[1];
			if (Operation.Compare(TEXT("disable"), ESearchCase::IgnoreCase) == 0)
			{
				bEnabled = false;
			}
			else if (Operation.Compare(TEXT("enable"), ESearchCase::IgnoreCase) != 0)
			{
				bIsOperationValid  = false;
			}
		}
		else
		{
			// Toggle by default
			bEnabled = !CsvProfiler->IsCategoryEnabled(CategoryIndex);
		}
		if (bIsOperationValid)
		{
			CsvProfiler->EnableCategoryByIndex(CategoryIndex, bEnabled);
			OutputDevice.Logf(ELogVerbosity::Log, TEXT("CsvProfiler: category '%s' is now %s."), *Category, bEnabled ? TEXT("enabled") : TEXT("disabled"));
			return;
		}
	}
	
	// We fall into here if there was a usage error
	OutputDevice.Logf(ELogVerbosity::Error, TEXT("CsvProfiler: Usage: csvcategory <category> [enable/disable] (toggles if second parameter is omitted)"));
}

static FAutoConsoleCommandWithWorldArgsAndOutputDevice HandleCSVCategoryCmd(
	TEXT("CsvCategory"),
	TEXT("Changes whether a CSV category is included in captures."),
	FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateStatic(&HandleCSVCategoryCommand)
);

static void HandleCSVDeleteDefaultDirectoryCommand()
{
	if ((GCsvProfilerIsCapturing && IsContinuousWriteEnabled(true)) || GCsvProfilerIsWritingFile)
	{
		UE_LOGF(LogCsvProfiler, Error, "CSV Profiler is writing file, ignoring command csv.DeleteDefaultDirectory");
	}
	else
	{
		IFileManager::Get().DeleteDirectory(*FCsvProfiler::GetDefaultDirectory(), false, true);
	}
}

static FAutoConsoleCommand HandleCSVDeleteDefaultDirectoryCmd(
	TEXT("csv.DeleteDefaultDirectory"),
	TEXT("Deletes the default directory for CSV Profiler files. Directory is currently only used by the CsvProfile command, and the command line argument: -ReplicationCSVCaptureFrames=X"),
	FConsoleCommandDelegate::CreateStatic(&HandleCSVDeleteDefaultDirectoryCommand)
);

//-----------------------------------------------------------------------------
//	TSingleProducerSingleConsumerList : fast lock-free single producer/single 
//  consumer list implementation. 
//  Uses a linked list of blocks for allocations.
//-----------------------------------------------------------------------------
template <class T, int BlockSize>
class TSingleProducerSingleConsumerList
{
	// A block of BlockSize entries
	struct FBlock
	{
		FBlock() : Next(nullptr)
		{
		}
		T Entries[BlockSize];

#if LIST_VALIDATION
		int32 DebugIndices[BlockSize];
#endif
		FBlock* Next;
	};

public:
	TSingleProducerSingleConsumerList()
	{
		HeadBlock = nullptr;
		TailBlock = nullptr;
#if DO_GUARD_SLOW
		bElementReserved = false;
#endif
#if LIST_VALIDATION
		LastDebugIndex = -1;
#endif
		Counter = 0;
		ConsumerThreadReadIndex = 0;
		ConsumerThreadDeleteIndex = 0;
	}

	~TSingleProducerSingleConsumerList()
	{
		// Only safe to destruct when no other threads are using the list.

		// Delete all remaining blocks in the list
		while (HeadBlock)
		{
			FBlock* PrevBlock = HeadBlock;
			HeadBlock = HeadBlock->Next;
			delete PrevBlock;
		}

		HeadBlock = nullptr;
		TailBlock = nullptr;
	}

	// Reserve an element prior to writing it
	// Must be called from the Producer thread
	CSV_PROFILER_INLINE T* ReserveElement()
	{
#if DO_GUARD_SLOW
		checkSlow(!bElementReserved);
		bElementReserved = true;
#endif
		uint32 TailBlockSize = Counter % BlockSize;
		if (TailBlockSize == 0)
		{
			AddTailBlock();
		}
#if LIST_VALIDATION
		TailBlock->DebugIndices[Counter % BlockSize] = Counter;
#endif
		return &TailBlock->Entries[TailBlockSize];
	}

	// Commit an element after writing it
	// Must be called from the Producer thread after a call to ReserveElement
	CSV_PROFILER_INLINE void CommitElement()
	{
#if DO_GUARD_SLOW
		checkSlow(bElementReserved);
		bElementReserved = false;
#endif
		FPlatformMisc::MemoryBarrier();

		// Keep track of the count of all the elements we ever committed. This value is never reset, even on a PopAll
		Counter++;
	}

	// Called from the consumer thread
	bool HasNewData() const
	{
		volatile uint64 CurrentCounterValue = Counter;
		FPlatformMisc::MemoryBarrier();
		return CurrentCounterValue > ConsumerThreadReadIndex;
	}

	// Called from the consumer thread
	void PopAll(TArray<T>& ElementsOut, int64 MaxSlackMemBytes = -1 )
	{
		volatile uint64 CurrentCounterValue = Counter;
		FPlatformMisc::MemoryBarrier();

		uint32 MaxElementsToPop = uint32(CurrentCounterValue - ConsumerThreadReadIndex);

		// Shrink the output array if it has excessive slack (note: usually this will shrink to 0)
		int64 SlackMemBytes = ( (int64)ElementsOut.GetSlack() - (int64)MaxElementsToPop ) * (int64)sizeof(T);
		if ( MaxSlackMemBytes >= 0 && SlackMemBytes > MaxSlackMemBytes )
		{
			ElementsOut.Shrink();
		}

		// Presize the array capacity to avoid memory reallocation.
		ElementsOut.Reserve(ElementsOut.Num() + MaxElementsToPop);

		uint32 IndexInBlock = ConsumerThreadReadIndex % BlockSize;

		for (uint32 Index = 0; Index < MaxElementsToPop; ++Index)
		{
			// if this block is full and it's completed, delete it and move to the next block (update the head)
			if (ConsumerThreadReadIndex == (ConsumerThreadDeleteIndex + BlockSize))
			{
				// Both threads are done with the head block now, so we can safely delete it 
				// Note that the Producer thread only reads/writes to the HeadBlock pointer on startup, so it's safe to update it at this point
				// HeadBlock->Next is also safe to read, since the producer can't be writing to it if Counter has reached this block
				FBlock* PrevBlock = HeadBlock;
				HeadBlock = HeadBlock->Next;
				IndexInBlock = 0;
				delete PrevBlock;

				ConsumerThreadDeleteIndex = ConsumerThreadReadIndex;
			}
			check(HeadBlock != nullptr);
			check(IndexInBlock < BlockSize);

			T& Element = HeadBlock->Entries[IndexInBlock];

			// Move construct. Avoids mem allocations on FString members
			ElementsOut.Emplace(MoveTemp(Element));

#if LIST_VALIDATION
			int32 DebugIndex = HeadBlock->DebugIndices[IndexInBlock];
			ensure(DebugIndex == LastDebugIndex + 1);
			LastDebugIndex = DebugIndex;
#endif
			IndexInBlock++;
			ConsumerThreadReadIndex++;
		}
	}

	inline uint64 GetAllocatedSize() const
	{
		volatile uint64 CurrentCounterValue = Counter;
		FPlatformMisc::MemoryBarrier();

		// Use the delete index, so we count all blocks that haven't been deleted yet.
		uint64 NumElements = CurrentCounterValue - ConsumerThreadDeleteIndex;
		uint64 NumBlocks = FMath::DivideAndRoundUp(NumElements, (uint64)BlockSize);

		return NumBlocks * sizeof(FBlock);
	}

private:
	void AddTailBlock()
	{
		LLM_SCOPE(ELLMTag::CsvProfiler);
		FBlock* NewTail = new FBlock;
		if (TailBlock == nullptr)
		{
			// This must only happen on startup, otherwise it's not thread-safe
			checkSlow(Counter == 0);
			checkSlow(HeadBlock == nullptr);
			HeadBlock = NewTail;
		}
		else
		{
			TailBlock->Next = NewTail;
		}
		TailBlock = NewTail;
	}


	FBlock* HeadBlock;
	FBlock* TailBlock;

	volatile uint64 Counter;

	// Used from the consumer thread
	uint64 ConsumerThreadReadIndex;
	uint64 ConsumerThreadDeleteIndex;

#if DO_GUARD_SLOW
	bool bElementReserved;
#endif
#if LIST_VALIDATION
	int32 LastDebugIndex;
#endif

};

namespace ECsvTimeline
{
	enum Type
	{
		Gamethread,
		Renderthread,
		EndOfPipe,
		Count
	};
}

//-----------------------------------------------------------------------------
//	FFrameBoundaries : thread-safe class for managing thread boundary timestamps
//  These timestamps are written from the gamethread/renderthread, and consumed
//  by the CSVProfiling thread
//-----------------------------------------------------------------------------
class FFrameBoundaries
{
public:
	FFrameBoundaries() : CurrentReadFrameIndex(0)
	{}

	void Clear()
	{
		check(IsInCsvProcessingThread());
		Update();
		for (int i = 0; i < ECsvTimeline::Count; i++)
		{
			FrameBoundaryTimestamps[i].Empty();
		}
		CurrentReadFrameIndex = 0;
		EOPCounter = 0;
	}

	int32 GetFrameNumberForTimestamp(ECsvTimeline::Type Timeline, uint64 Timestamp) const
	{
		// If we have new frame data pending, grab it now
		if (FrameBoundaryTimestampsWriteBuffer[Timeline].HasNewData())
		{
			const_cast<FFrameBoundaries*>(this)->Update(Timeline);
		}

		const TArray<uint64>& ThreadTimestamps = FrameBoundaryTimestamps[Timeline];
		if (ThreadTimestamps.Num() == 0 || Timestamp < ThreadTimestamps[0])
		{
			// This timestamp is before the first frame, or there are no valid timestamps
			CurrentReadFrameIndex = 0;
			return -1;
		}

		if (CurrentReadFrameIndex >= ThreadTimestamps.Num())
		{
			CurrentReadFrameIndex = ThreadTimestamps.Num() - 1;
		}


		// Check if we need to rewind
		if (CurrentReadFrameIndex > 0 && ThreadTimestamps[CurrentReadFrameIndex - 1] > Timestamp)
		{
			// Binary search to < 4 and then resume linear searching
			int32 StartPos = 0;
			int32 EndPos = CurrentReadFrameIndex;
			while (true)
			{
				int32 Diff = (EndPos - StartPos);
				if (Diff <= 4)
				{
					CurrentReadFrameIndex = StartPos;
					break;
				}
				int32 MidPos = (EndPos + StartPos) / 2;
				if (ThreadTimestamps[MidPos] > Timestamp)
				{
					EndPos = MidPos;
				}
				else
				{
					StartPos = MidPos;
				}
			}
		}

		for (; CurrentReadFrameIndex < ThreadTimestamps.Num(); CurrentReadFrameIndex++)
		{
			if (Timestamp < ThreadTimestamps[CurrentReadFrameIndex])
			{
				// Might return -1 if this was before the first frame
				return CurrentReadFrameIndex - 1;
			}
		}
		return ThreadTimestamps.Num() - 1;
	}

	uint64 AddBeginFrameTimestamp(ECsvTimeline::Type Timeline, const bool bDoThreadCheck = true)
	{
#if DO_CHECK
		if (bDoThreadCheck)
		{
			switch (Timeline)
			{
			case ECsvTimeline::Gamethread:
				check(IsInGameThread());
				break;
			case ECsvTimeline::Renderthread:
				check(IsInRenderingThread());
				break;
			}
		}
#endif
		//
		// The EndOfPipe frame boundary happens more frequently than the game or render thread frame boundaries.
		// This is because EndOfPipe happens on each RHICmdList.EndFrame() call, which includes the loading screen / video player when the game thread is not ticking.
		// Use a counter to ignore any EndOfPipe boundaries if we haven't had a prior render thread boundary. Not doing this means the timelines get out of sync.
		//
		if (Timeline == ECsvTimeline::Renderthread)
		{
			EOPCounter++;
		}
		else if (Timeline == ECsvTimeline::EndOfPipe)
		{
			int32 Counter = EOPCounter--;
			if (Counter == 0)
			{
				// Skip this frame boundary.
				EOPCounter++;
				return 0;
			}
		}

		uint64 Timestamp = FPlatformTime::Cycles64();
		uint64* Element = FrameBoundaryTimestampsWriteBuffer[Timeline].ReserveElement();
		*Element = Timestamp;
		FrameBoundaryTimestampsWriteBuffer[Timeline].CommitElement();
		return Timestamp;
	}

private:
	void Update(ECsvTimeline::Type Timeline = ECsvTimeline::Count)
	{
		check(IsInCsvProcessingThread());
		if (Timeline == ECsvTimeline::Count)
		{
			for (int32 i = 0; i < int32(ECsvTimeline::Count); i++)
			{
				FrameBoundaryTimestampsWriteBuffer[i].PopAll(FrameBoundaryTimestamps[i]);
			}
		}
		else
		{
			FrameBoundaryTimestampsWriteBuffer[Timeline].PopAll(FrameBoundaryTimestamps[Timeline]);
		}
	}

	TSingleProducerSingleConsumerList<uint64, 16> FrameBoundaryTimestampsWriteBuffer[ECsvTimeline::Count];
	TArray<uint64> FrameBoundaryTimestamps[ECsvTimeline::Count];
	mutable int32 CurrentReadFrameIndex;

	std::atomic<int32> EOPCounter { 0 };
};
static FFrameBoundaries GFrameBoundaries;


static TMap<const ANSICHAR*, uint32> CharPtrToStringIndex;
static TMap<FString, uint32> UniqueNonFNameStatIDStrings;
static TArray<FString> UniqueNonFNameStatIDIndices;

struct FAnsiStringRegister
{
	static uint32 GetUniqueStringIndex(const ANSICHAR* AnsiStr)
	{
		uint32* IndexPtr = CharPtrToStringIndex.Find(AnsiStr);
		if (IndexPtr)
		{
			return *IndexPtr;
		}

		// If we haven't seen this pointer before, check the string register (this is slow!)
		FString Str = FString(StringCast<TCHAR>(AnsiStr).Get());
		uint32* Value = UniqueNonFNameStatIDStrings.Find(Str);
		if (Value)
		{
			// Cache in the index register
			CharPtrToStringIndex.Add(AnsiStr, *Value);
			return *Value;
		}
		// Otherwise, this string is totally new
		uint32 NewIndex = UniqueNonFNameStatIDIndices.Num();
		UniqueNonFNameStatIDStrings.Add(Str, NewIndex);
		UniqueNonFNameStatIDIndices.Add(Str);
		CharPtrToStringIndex.Add(AnsiStr, NewIndex);
		return NewIndex;
	}

	static FString GetString(uint32 Index)
	{
		return UniqueNonFNameStatIDIndices[Index];
	}
};


class FCsvStatRegister
{
	static const uint64 FNameOrIndexMask = 0x0007ffffffffffffull; // Lower 51 bits for fname or index

	struct FStatIDFlags
	{
		static const uint8 IsCountStat = 0x01;
	};

public:
	FCsvStatRegister()
	{
		Clear();
	}

	int32 GetUniqueIndex(uint64 InStatIDRaw, int32 InCategoryIndex, bool bInIsFName, bool bInIsCountStat)
	{
		check(IsInCsvProcessingThread());

		// Make a compound key
		FCsvUniqueStatID UniqueID(InStatIDRaw, InCategoryIndex, bInIsFName, bInIsCountStat);

		uint64 Hash = UniqueID.Hash;
		int32 *IndexPtr = StatIDToIndex.Find(Hash);
		if (IndexPtr)
		{
			return *IndexPtr;
		}
		else
		{
			int32 IndexOut = -1;
			FString NameStr;
			if (bInIsFName)
			{
				check((InStatIDRaw & FNameOrIndexMask) == InStatIDRaw);
				const FNameEntry* NameEntry = FName::GetEntry(FNameEntryId::FromUnstableInt(UniqueID.Fields.FNameOrIndex));
				NameStr = NameEntry->GetPlainNameString();
			}
			else
			{
				// With non-fname stats, the same string can appear with different pointers.
				// We need to look up the stat in the ansi stat register to see if it's actually unique
				uint32 AnsiNameIndex = FAnsiStringRegister::GetUniqueStringIndex((ANSICHAR*)InStatIDRaw);
				FCsvUniqueStatID AnsiUniqueID(UniqueID);
				AnsiUniqueID.Fields.FNameOrIndex = AnsiNameIndex;
				int32 *AnsiIndexPtr = AnsiStringStatIDToIndex.Find(AnsiUniqueID.Hash);
				if (AnsiIndexPtr)
				{
					// This isn't a new stat. Only the pointer is new, not the string itself
					IndexOut = *AnsiIndexPtr;
					// Update the master lookup table
					StatIDToIndex.Add(UniqueID.Hash, IndexOut);
					return IndexOut;
				}
				else
				{
					// Stat has never been seen before. Add it to the ansi map
					AnsiStringStatIDToIndex.Add(AnsiUniqueID.Hash, StatIndexCount);
				}
				NameStr = FAnsiStringRegister::GetString(AnsiNameIndex);
			}

			// Store the index in the master map
			IndexOut = StatIndexCount;
			StatIDToIndex.Add( UniqueID.Hash,  IndexOut);
			StatIndexCount++;

			// Store the name, category index and flags
			StatNames.Add(NameStr);
			StatCategoryIndices.Add(InCategoryIndex);

			uint8 Flags = 0;
			if (bInIsCountStat)
			{
				Flags |= FStatIDFlags::IsCountStat;
			}
			StatFlags.Add(Flags);

			return IndexOut;
		}
	}

	void Clear()
	{
		StatIndexCount = 0;
		StatIDToIndex.Reset();
		AnsiStringStatIDToIndex.Reset();
		StatNames.Empty();
		StatCategoryIndices.Empty();
		StatFlags.Empty();
	}

	const FString& GetStatName(int32 Index) const
	{
		return StatNames[Index];
	}
	int32 GetCategoryIndex(int32 Index) const
	{
		return StatCategoryIndices[Index];
	}

	bool IsCountStat(int32 Index) const
	{
		return !!(StatFlags[Index] & FStatIDFlags::IsCountStat);
	}

protected:
	TMap<uint64, int32> StatIDToIndex;
	TMap<uint64, int32> AnsiStringStatIDToIndex;
	uint32 StatIndexCount;
	TArray<FString> StatNames;
	TArray<int32> StatCategoryIndices;
	TArray<uint8> StatFlags;
};

//-----------------------------------------------------------------------------
//	FCsvTimingMarker : records timestamps. Uses StatName pointer as a unique ID
//-----------------------------------------------------------------------------
struct FCsvStatBase
{
	struct FFlags
	{
		static const uint8 StatIDIsFName = 0x01;
		static const uint8 TimestampBegin = 0x02;
		static const uint8 IsCustomStat = 0x04;
		static const uint8 IsInteger = 0x08;
		static const uint8 IsExclusiveTimestamp = 0x10;
		static const uint8 IsExclusiveInsertedMarker = 0x20;
	};

	CSV_PROFILER_INLINE void Init(uint64 InStatID, int32 InCategoryIndex, uint8 InFlags, uint64 InTimestamp)
	{
		Timestamp = InTimestamp;
		Flags = InFlags;
		RawStatID = InStatID;
		CategoryIndex = InCategoryIndex;
	}

	CSV_PROFILER_INLINE void Init(uint64 InStatID, int32 InCategoryIndex, uint8 InFlags, uint64 InTimestamp, uint8 InUserData)
	{
		Timestamp = InTimestamp;
		RawStatID = InStatID;
		CategoryIndex = InCategoryIndex;
		UserData = InUserData;
		Flags = InFlags;
	}

	CSV_PROFILER_INLINE uint32 GetUserData() const
	{
		return UserData;
	}

	CSV_PROFILER_INLINE uint64 GetTimestamp() const
	{
		return Timestamp;
	}

	CSV_PROFILER_INLINE bool IsCustomStat() const
	{
		return !!(Flags & FCsvStatBase::FFlags::IsCustomStat);
	}

	CSV_PROFILER_INLINE bool IsFNameStat() const
	{
		return !!(Flags & FCsvStatBase::FFlags::StatIDIsFName);
	}

	uint64 Timestamp;

	// Use with caution! In the case of non-fname stats, strings from different scopes may will have different RawStatIDs (in that case RawStatID is simply a char * cast to a uint64). 
	// Use GetSeriesStatID() (slower) to get a unique ID for a string where needed
	uint64 RawStatID;
	int32 CategoryIndex;

	uint8 UserData;
	uint8 Flags;
};

struct FCsvTimingMarker : public FCsvStatBase
{
	bool IsBeginMarker() const
	{
		return !!(Flags & FCsvStatBase::FFlags::TimestampBegin);
	}
	bool IsExclusiveMarker() const
	{
		return !!(Flags & FCsvStatBase::FFlags::IsExclusiveTimestamp);
	}
	bool IsExclusiveArtificialMarker() const
	{
		return !!(Flags & FCsvStatBase::FFlags::IsExclusiveInsertedMarker);
	}
};

struct FCsvCustomStat : public FCsvStatBase
{
	ECsvCustomStatOp GetCustomStatOp() const
	{
		return (ECsvCustomStatOp)GetUserData();
	}

	bool IsInteger() const
	{
		return !!(Flags & FCsvStatBase::FFlags::IsInteger);
	}

	double GetValueAsDouble() const
	{
		return IsInteger() ? double(Value.AsInt) : double(Value.AsFloat);
	}

	union FValue
	{
		float AsFloat;
		uint32 AsInt;
	} Value;
};

struct FCsvEvent
{
	inline uint64 GetAllocatedSize() const { return (uint64)EventText.GetAllocatedSize(); }

	FString EventText;
	uint64 Timestamp;
	uint32 CategoryIndex;
};


struct FCsvStatSeriesValue
{
	FCsvStatSeriesValue() { Value.AsInt = 0; }
	union
	{
		int32 AsInt;
		float AsFloat;
	} Value;
};


class FCsvWriterHelper
{
public:
	FCsvWriterHelper(const TSharedRef<FArchive>& InOutputFile, int32 InBufferSize, bool bInCompressOutput, bool bInContinuousWrites)
		: OutputFile(InOutputFile)
		, bIsLineStart(true)
		, BytesInBuffer(0)
		, MaxFileBytes(0)
	{
		if (InBufferSize > 0)
		{
			Buffer.SetNumUninitialized(InBufferSize);
			if (bInCompressOutput)
			{
				GZipBuffer.SetNumUninitialized(InBufferSize);
			}
		}

		BytesWritten.store(0);

		int32 MaxFileSizeMB = GetMaxFileSizeMB(false);
		if (MaxFileSizeMB > 0)
		{
			if (!bInContinuousWrites)
			{
				UE_LOGF(LogCsvProfiler, Error, "cvar csv.MaxFileSizeMB not supported unless the appropriate ContinuousWrites cvar is true. Ignoring MaxFileSizeMB.");
				MaxFileBytes = 0;
			}
			else
			{
				MaxFileBytes = MaxFileSizeMB;
				MaxFileBytes *= 1024 * 1024;
				if (MaxFileBytes > InBufferSize)
				{
					// reduce the threshold so that tuned value isn't exceeded quite as much once the file is done flushing
					MaxFileBytes -= InBufferSize;
				}
			}
		}
	}

	~FCsvWriterHelper()
	{
		Flush();
	}

	void WriteSemicolonSeparatedStringList(const TArray<FString>& Strings)
	{
		WriteEmptyString();

		for (int32 Index = 0; Index < Strings.Num(); ++Index)
		{
			FString SanitizedText = Strings[Index];

			// Remove semi-colons and commas from event strings so we can safely separate using them
			SanitizedText.ReplaceInline(TEXT(";"), TEXT("."));
			SanitizedText.ReplaceInline(TEXT(","), TEXT("."));

			if (Index > 0)
			{
				WriteChar(';');
			}

			WriteStringInternal(SanitizedText);
		}
	}

	void NewLine()
	{
		WriteChar('\n');
		bIsLineStart = true;
	}

	void WriteString(const FString& Str)
	{
		if (!bIsLineStart)
		{
			WriteChar(',');
		}
		bIsLineStart = false;
		WriteStringInternal(Str);
	}

	void WriteEmptyString()
	{
		if (!bIsLineStart)
		{
			WriteChar(',');
		}
		bIsLineStart = false;
	}

	void WriteValue(double Value)
	{
		if (!bIsLineStart)
		{
			WriteChar(',');
		}
		bIsLineStart = false;

		int32 StrLen;
		ANSICHAR StringBuffer[256];

		if (FMath::Frac((float)Value) == 0.0f)
		{
			StrLen = FCStringAnsi::Snprintf(StringBuffer, 256, "%.0f", Value);
		}
		else if (FMath::Abs(Value) < 0.1)
		{
			StrLen = FCStringAnsi::Snprintf(StringBuffer, 256, "%.6f", Value);
		}
		else
		{
			StrLen = FCStringAnsi::Snprintf(StringBuffer, 256, "%.4f", Value);
		}
		SerializeInternal((void*)StringBuffer, sizeof(ANSICHAR) * StrLen);
	}

	void WriteMetadataEntry(const FString& Key, const FString& Value)
	{
		WriteString(*FString::Printf(TEXT("[%s]"), *Key));
		WriteString(Value);
	}

private:
	void WriteStringInternal(const FString& Str)
	{
		auto AnsiStr = StringCast<ANSICHAR>(*Str);
		SerializeInternal((void*)AnsiStr.Get(), AnsiStr.Length());
	}

	void WriteChar(ANSICHAR Char)
	{
		SerializeInternal((void*)&Char, sizeof(ANSICHAR));
	}

	void SerializeInternal(void* Src, int32 NumBytes)
	{
		if (Buffer.Num() == 0)
		{
			SerializeToOutputFile(Src, NumBytes);
		}
		else
		{
			uint8* SrcPtr = (uint8*)Src;

			while (NumBytes)
			{
				int32 BytesToWrite = FMath::Min(Buffer.Num() - BytesInBuffer, NumBytes);
				if (BytesToWrite == 0)
				{
					Flush();
				}
				else
				{
					FMemory::Memcpy(&Buffer[BytesInBuffer], SrcPtr, BytesToWrite);
					BytesInBuffer += BytesToWrite;
					SrcPtr += BytesToWrite;
					NumBytes -= BytesToWrite;
				}
			}
		}
	}

	void Flush()
	{
		if (BytesInBuffer > 0)
		{
			if (GZipBuffer.Num() > 0)
			{
				// Compression is enabled.
				int32 CompressedSize;
				{
					while (true)
					{
						// Compress the data in Buffer into the GZipBuffer array
						CompressedSize = GZipBuffer.Num();
						if (FCompression::CompressMemory(
							NAME_Gzip,
							GZipBuffer.GetData(), CompressedSize,
							Buffer.GetData(), BytesInBuffer,
							ECompressionFlags::COMPRESS_BiasSpeed))
						{
							break;
						}

						// Compression failed.
						if (CompressedSize > GZipBuffer.Num())
						{
							// Failed because the buffer size was too small. Increase the buffer size.
							GZipBuffer.SetNumUninitialized(CompressedSize);
						}
						else
						{
							// Buffer was already large enough. Unknown error. Nothing we can do here but discard the data.
							UE_LOGF(LogCsvProfiler, Error, "CSV data compression failed.");
							BytesInBuffer = 0;
							return;
						}
					}
				}

				{
					SerializeToOutputFile(GZipBuffer.GetData(), CompressedSize);
				}
			}
			else
			{
				// No compression. Write directly to the output file
				SerializeToOutputFile(Buffer.GetData(), BytesInBuffer);
			}

			BytesInBuffer = 0;
		}
	}

	void SerializeToOutputFile(void* Src, int32 NumBytes)
	{
		OutputFile->Serialize(Src, NumBytes);

		uint64 PreviousBytesWritten = BytesWritten.fetch_add(NumBytes, std::memory_order_relaxed);
		if (MaxFileBytes > 0)
		{
			uint64 CurrentBytesWritten = PreviousBytesWritten;
			CurrentBytesWritten += NumBytes;
			if (PreviousBytesWritten < MaxFileBytes && CurrentBytesWritten >= MaxFileBytes)
			{
				UE_LOGF(LogCsvProfiler, Log, "CSV Profiler will no longer write values due to exceeding csv.MaxFileSizeMB. %llu written of %llu max bytes)", CurrentBytesWritten, MaxFileBytes);
			}
		}
	}

	TSharedRef<FArchive> OutputFile;
	bool bIsLineStart;

	int32 BytesInBuffer;
	TArray<uint8> Buffer;
	TArray<uint8> GZipBuffer;

	std::atomic<uint64> BytesWritten;
	uint64 MaxFileBytes;

public:
	inline uint64 GetAllocatedSize() const
	{
		return Buffer.GetAllocatedSize() + GZipBuffer.GetAllocatedSize();
	}

	inline uint64 GetNumBytesWritten() const
	{
		return BytesWritten.load(std::memory_order_relaxed);
	}

	inline bool HasReachedMaxFileBytes() const
	{
		return MaxFileBytes <= 0 ? false : GetNumBytesWritten() >= MaxFileBytes;
	}
};

struct FCsvProcessedEvent
{
	inline uint64 GetAllocatedSize() const { return EventText.GetAllocatedSize(); }

	FString GetFullText() const
	{
		// If the event has a timestamp, output it after the text with a ## separator
		if (TimeOffsetSeconds != std::numeric_limits<double>::max())
		{
			if (CategoryIndex == 0)
			{
				return FString::Printf(TEXT("%s##%.6f"), *EventText, TimeOffsetSeconds);
			}
			return FString::Printf(TEXT("%s/%s##%.6f"), *FCsvCategoryData::Get()->GetCategoryNameByIndex(CategoryIndex), *EventText, TimeOffsetSeconds);
		}
		if (CategoryIndex == 0)
		{
			return EventText;
		}
		return FString::Printf(TEXT("%s/%s"), *FCsvCategoryData::Get()->GetCategoryNameByIndex(CategoryIndex), *EventText);
	}

	FString EventText;
	double TimeOffsetSeconds;
	uint32 FrameNumber;
	uint32 CategoryIndex;
};


class FCsvStatNameValidator
{
public:
	FCsvStatNameValidator()
	{
		ValidCharacters = TEXT("0123456789abcdefghijklmnopqrstuvwxyzABCDEFGHIJKLMNOPQRSTUVWXYZ /_-[]()#.:");
		for (TCHAR Character : ValidCharacters)
		{
			ValidCharactersSet.Add(Character);
		}
	}
	bool IsNameValid(const FString& StatName) const
	{
		for (TCHAR Character : StatName)
		{
			if (!ValidCharactersSet.Contains(Character))
			{
				return false;
			}
		}
		return true;
	}

	FString SanitizeName(const FString& StatName) const
	{
		FString NameOut;
		for (TCHAR Character : StatName)
		{
			if (ValidCharactersSet.Contains(Character))
			{
				NameOut += Character;
			}
		}
		return NameOut;
	}

	const FString& GetValidCharacters() const
	{
		return ValidCharacters;
	}
private:
	TSet<TCHAR> ValidCharactersSet;
	FString ValidCharacters;
};
static FCsvStatNameValidator* GCsvStatNameValidator = nullptr;

typedef int32 FCsvStatID;

struct FCsvStatSeries
{
	enum class EType : uint8
	{
		TimerData,
		CustomStatInt,
		CustomStatFloat
	};

	FCsvStatSeries(EType InSeriesType, const FCsvStatID& InStatID, FCsvStreamWriter* InWriter, FCsvStatRegister& StatRegister, const FString& ThreadName, FCsvAggregateStatSeries* InLinkedAggregateStatSeries);
	
	virtual ~FCsvStatSeries()
	{
	}

	void FlushIfDirty();

	void SetTimerValue(uint32 DataFrameNumber, uint64 ElapsedCycles)
	{
		check(SeriesType == EType::TimerData);
		ensure(CurrentWriteFrameNumber <= DataFrameNumber || CurrentWriteFrameNumber == -1);

		// If we're done with the previous frame, commit it
		if (CurrentWriteFrameNumber != DataFrameNumber)
		{
			if (CurrentWriteFrameNumber != -1)
			{
				FlushIfDirty();
			}
			CurrentWriteFrameNumber = DataFrameNumber;
			bDirty = true;
		}
		CurrentValue.AsTimerCycles += ElapsedCycles;
	}

	void SetCustomStatValue_Int(uint32 DataFrameNumber, ECsvCustomStatOp Op, int32 Value)
	{
		check(SeriesType == EType::CustomStatInt);
		ensure(CurrentWriteFrameNumber <= DataFrameNumber || CurrentWriteFrameNumber == -1);

		// Is this a new frame?
		if (CurrentWriteFrameNumber != DataFrameNumber)
		{
			// If we're done with the previous frame, commit it
			if (CurrentWriteFrameNumber != -1)
			{
				FlushIfDirty();
			}

			// The first op in a frame is always a set. Otherwise min/max don't work
			Op = ECsvCustomStatOp::Set;
			CurrentWriteFrameNumber = DataFrameNumber;
			bDirty = true;
		}

		switch (Op)
		{
		case ECsvCustomStatOp::Set:
			CurrentValue.AsIntValue = Value;
			break;

		case ECsvCustomStatOp::Min:
			CurrentValue.AsIntValue = FMath::Min(Value, CurrentValue.AsIntValue);
			break;

		case ECsvCustomStatOp::Max:
			CurrentValue.AsIntValue = FMath::Max(Value, CurrentValue.AsIntValue);
			break;
		case ECsvCustomStatOp::Accumulate:
			CurrentValue.AsIntValue += Value;
			break;
		}
	}

	void SetCustomStatValue_Float(uint32 DataFrameNumber, ECsvCustomStatOp Op, float Value)
	{
		check(SeriesType == EType::CustomStatFloat);
		ensure(CurrentWriteFrameNumber <= DataFrameNumber || CurrentWriteFrameNumber == -1);

		// Is this a new frame?
		if (CurrentWriteFrameNumber != DataFrameNumber)
		{
			// If we're done with the previous frame, commit it
			if (CurrentWriteFrameNumber != -1)
			{
				FlushIfDirty();
			}

			// The first op in a frame is always a set. Otherwise min/max don't work
			Op = ECsvCustomStatOp::Set;
			CurrentWriteFrameNumber = DataFrameNumber;
			bDirty = true;
		}

		switch (Op)
		{
		case ECsvCustomStatOp::Set:
			CurrentValue.AsFloatValue = Value;
			break;

		case ECsvCustomStatOp::Min:
			CurrentValue.AsFloatValue = FMath::Min(Value, CurrentValue.AsFloatValue);
			break;

		case ECsvCustomStatOp::Max:
			CurrentValue.AsFloatValue = FMath::Max(Value, CurrentValue.AsFloatValue);
			break;

		case ECsvCustomStatOp::Accumulate:
			CurrentValue.AsFloatValue += Value;
			break;
		}
	}

	bool IsCustomStat() const
	{
		return (SeriesType == EType::CustomStatFloat || SeriesType == EType::CustomStatInt);
	}

	virtual void FinalizeFrame(int64 FrameNumber)
	{
		// Stat values are held in the series until a new value arrives.
		// If we've caught up with the last value written to the series,
		// we need to flush to get the correct value for this frame.
		if (CurrentWriteFrameNumber == FrameNumber)
		{
			FlushIfDirty();
		}
	}

	virtual bool IsAggregateSeries() const
	{
		return false;
	}

	virtual uint64 GetAllocatedSize() const 
	{ 
		return sizeof(*this) + Name.GetAllocatedSize();
	}

	FCsvStatID StatID;
	const EType SeriesType;
	FString Name;

	uint32 CurrentWriteFrameNumber;
	union
	{
		int32   AsIntValue;
		float   AsFloatValue;
		uint64  AsTimerCycles;
	} CurrentValue;

	FCsvStreamWriter* Writer;

	int32 ColumnIndex;

	FCsvAggregateStatSeries* LinkedAggregateStatSeries;

	bool bDirty;
};


struct FCsvAggregateStatSeries : public FCsvStatSeries
{
	FCsvAggregateStatSeries(EType InSeriesType, const FCsvStatID& InStatID, FCsvStreamWriter* InWriter, FCsvStatRegister& StatRegister, const FString& ThreadName) 
		: FCsvStatSeries(InSeriesType, InStatID, InWriter, StatRegister, ThreadName, nullptr)
	{
	}

	// Accumulate the value for the linked series for a given frame
	void AccumulateLinkedSeriesValue(int64 FrameNumber, const FCsvStatSeriesValue& Value)
	{
		FCsvStatSeriesValue& FrameValue = RowValues.FindOrAdd(FrameNumber);
		switch (SeriesType)
		{
		case EType::TimerData:
		case EType::CustomStatFloat:
			FrameValue.Value.AsFloat += Value.Value.AsFloat;
			break;
		case EType::CustomStatInt:
			FrameValue.Value.AsInt += Value.Value.AsInt;
		}
	}

	virtual void FinalizeFrame(int64 FrameNumber) override;

	virtual bool IsAggregateSeries() const override
	{
		return true;
	}

	virtual uint64 GetAllocatedSize() const override
	{
		return ((uint64)RowValues.GetAllocatedSize());
	}

	TMap<int64, FCsvStatSeriesValue> RowValues;
};

class FCsvThreadGroupStatProcessor;

class FCsvStreamWriter
{
	struct FCsvRow
	{
		TArray<FCsvStatSeriesValue> Values;
		TArray<FCsvProcessedEvent> Events;

		inline uint64 GetAllocatedSize() const
		{
			uint64 Size = Values.GetAllocatedSize() + Events.GetAllocatedSize();
			for (const FCsvProcessedEvent& Event : Events)
			{
				Size += Event.GetAllocatedSize();
			}
			return Size;
		}
	};

	TMap<int64, FCsvRow> Rows;
	FCsvWriterHelper Stream;

	// There is no way to know what a frame is completed, to flush a CSV row to disk. Instead, we track the maximum
	// frame index we've seen from CSV data processing (WriteFrameIndex) and choose to flush all rows that have a
	// frame index less than (WriteFrameIndex - NumFramesToBuffer). NumFramesToBuffer should be large enough to avoid
	// flushing rows before all the timestamps for that frame have been processed, but small enough to avoid the
	// additional memory overhead of holding addition rows in memory unnecessarily.
	int64 NumFramesToBuffer;
	int64 WriteFrameIndex;
	int64 ReadFrameIndex;

	const bool bContinuousWrites;
	bool bFirstRow;

	// All series, including hidden series that feed into aggregate series rather than writing to the CSV directly
	TArray<FCsvStatSeries*> AllSeries;
	
	// Subset of AllSeries - only visible
	TArray<FCsvStatSeries*> VisibleSeries;

	TArray<class FCsvProfilerThreadDataProcessor*> DataProcessors;

	TSharedPtr<FCsvThreadGroupStatProcessor> TaskWorkerThreadGroupStatProcessor;

	uint32 RenderThreadId;
	uint32 RHIThreadId;

public:
	FCsvStreamWriter(const TSharedRef<FArchive>& InOutputFile, bool bInContinuousWrites, int32 InBufferSize, int64 InNumFramesToBuffer, bool bInCompressOutput, uint32 RenderThreadId, uint32 RHIThreadId, bool bAggregateTaskWorkerStats);
	~FCsvStreamWriter();

	void AddSeries(FCsvStatSeries* Series, bool bIsVisible);

	void PushValue(FCsvStatSeries* Series, int64 FrameNumber, const FCsvStatSeriesValue& Value);
	void PushEvent(const FCsvProcessedEvent& Event);

	void FinalizeNextRow();
	void Process(FCsvProcessThreadDataStats& OutStats);

	void Finalize(const TMap<FString, FString>& Metadata);

	inline uint64 GetAllocatedSize() const;

	inline uint64 GetNumBytesWritten() const;
	inline bool HasReachedMaxFileBytes() const;
};

FCsvStatSeries::FCsvStatSeries(EType InSeriesType, const FCsvStatID& InStatID, FCsvStreamWriter* InWriter, FCsvStatRegister& StatRegister, const FString& ThreadName, FCsvAggregateStatSeries* InLinkedAggregateStatSeries)
	: StatID(InStatID)
	, SeriesType(InSeriesType)
	, CurrentWriteFrameNumber(-1)
	, Writer(InWriter)
	, ColumnIndex(-1)
	, LinkedAggregateStatSeries(InLinkedAggregateStatSeries)
	, bDirty(false)
{
	CurrentValue.AsTimerCycles = 0;

	int32 StatCategoryIndex = StatRegister.GetCategoryIndex(StatID);

	Name = StatRegister.GetStatName(StatID);
	bool bIsCountStat = StatRegister.IsCountStat(StatID);

	if (!IsCustomStat() || bIsCountStat || LinkedAggregateStatSeries != nullptr || (CSV_DEBUG_CUSTOM_STATS_INCLUDE_THREAD_NAME==1))
	{
		// Add a /<Threadname> prefix
		Name = ThreadName + TEXT("/") + Name;
	}

	if (StatCategoryIndex > 0)
	{
		// Categorized stats are prefixed with <CATEGORY>/
		Name = FCsvCategoryData::Get()->GetCategoryNameByIndex(StatCategoryIndex) + TEXT("/") + Name;
	}

	if (bIsCountStat)
	{
		// Add a counts prefix
		Name = TEXT("COUNTS/") + Name;
	}

	// If we have a linked stat series, don't write directly to the CSV
	bool bVisible = LinkedAggregateStatSeries == nullptr || (CSV_DEBUG_EMIT_SEPARATE_THREAD_STATS_WHEN_TASK_AGGREGATION_ENABLED==1);

	// Name validation
	if (CVarCsvStatNameValidation.GetValueOnAnyThread() > 0 && GCsvStatNameValidator != nullptr && !GCsvStatNameValidator->IsNameValid(Name))
	{
		UE_LOGF(LogCsvProfiler, Warning, "Stat name '%ls' contains invalid characters. Valid characters are: '%ls'", *Name, *GCsvStatNameValidator->GetValidCharacters());
		if (CVarCsvStatNameValidation.GetValueOnAnyThread() == 2)
		{
			Name = GCsvStatNameValidator->SanitizeName(Name);
		}
		else if (CVarCsvStatNameValidation.GetValueOnAnyThread() == 3)
		{
			bVisible = false;
		}
	}

	// Internal CsvBench stats should be hidden
	if (StatCategoryIndex == CSV_CATEGORY_INDEX(CsvBench))
	{
		bVisible = false;
	}

	Writer->AddSeries(this, bVisible);
}

void FCsvStatSeries::FlushIfDirty()
{
	if (bDirty)
	{
		FCsvStatSeriesValue Value;
		switch (SeriesType)
		{
		case EType::TimerData:
			Value.Value.AsFloat = (float)FPlatformTime::ToMilliseconds64(CurrentValue.AsTimerCycles);
			break;
		case EType::CustomStatInt:
			Value.Value.AsInt = CurrentValue.AsIntValue;
			break;
		case EType::CustomStatFloat:
			Value.Value.AsFloat = CurrentValue.AsFloatValue;
			break;
		}
		// If there's an aggregate stat, push data to that rather than the CSV writer directly
		if (LinkedAggregateStatSeries)
		{
			LinkedAggregateStatSeries->AccumulateLinkedSeriesValue(CurrentWriteFrameNumber, Value);
		}
		// If this series is hidden then don't write
		if (ColumnIndex != -1)
		{
			Writer->PushValue(this, CurrentWriteFrameNumber, Value);
		}
		CurrentValue.AsTimerCycles = 0;
		bDirty = false;
	}
}

void FCsvAggregateStatSeries::FinalizeFrame(int64 FrameNumber)
{
	// Write the final value for this row to the writer
	FCsvStatSeriesValue Value;
	if (RowValues.RemoveAndCopyValue(FrameNumber, Value))
	{
		Writer->PushValue(this, FrameNumber, Value);
	}
}

struct FCsvWaitStatName
{
	explicit constexpr FCsvWaitStatName(
		const char* InStatName,
		const char* InFormattedStatName = nullptr,
		const char* InFormattedStatNameNonCP = nullptr)
		: StatName(InStatName)
		, FormattedStatName(InFormattedStatName)
		, FormattedStatNameNonCP(InFormattedStatNameNonCP)
	{
	}

	constexpr friend bool operator==(const FCsvWaitStatName& Lhs, const FCsvWaitStatName& Rhs)
	{
		return Lhs.StatName == Rhs.StatName;
	}

	const char* StatName = nullptr;
	const char* FormattedStatName = nullptr;
	const char* FormattedStatNameNonCP = nullptr;
};

static constexpr FCsvWaitStatName GDefaultWaitStatName("EventWait");
static constexpr FCsvWaitStatName GIgnoreWaitStatName("[IGNORE]");

class FCsvProfilerThreadData
{
public:
	typedef TWeakPtr<FCsvProfilerThreadData, ESPMode::ThreadSafe> FWeakPtr;
	typedef TSharedPtr<FCsvProfilerThreadData, ESPMode::ThreadSafe> FSharedPtr;

private:
	static CSV_PROFILER_INLINE uint64 GetStatID(const char* StatName) { return uint64(StatName); }
	static CSV_PROFILER_INLINE uint64 GetStatID(const FName& StatId) { return StatId.ToUnstableInt(); }

	// Group these objects together to enforce correct global-scope object destruction order.
	struct FSingleton
	{
		FCriticalSection TlsCS;
		TArray<FWeakPtr> TlsInstances;
		uint32 TlsSlot = FPlatformTLS::InvalidTlsSlot;
	} static Singleton;

public:
	static void InitTls()
	{
		if (!FPlatformTLS::IsValidTlsSlot(Singleton.TlsSlot))
		{
			Singleton.TlsSlot = FPlatformTLS::AllocTlsSlot();
			FPlatformMisc::MemoryBarrier();
		}
	}

	static bool IsTlsSlotInitialized()
	{
		return FPlatformTLS::IsValidTlsSlot(Singleton.TlsSlot);
	}

	static FORCENOINLINE FCsvProfilerThreadData* CreateTLSData()
	{
		QUICK_SCOPE_CYCLE_COUNTER(CSVProfiler_ThreadData_CreateTLSData);
		UE::TScopeLock Lock(Singleton.TlsCS);

		FSharedPtr ProfilerThreadPtr = MakeShareable(new FCsvProfilerThreadData());
		FPlatformTLS::SetTlsValue(Singleton.TlsSlot, ProfilerThreadPtr.Get());

		// Keep a weak reference to this thread data in the global array.
		Singleton.TlsInstances.Emplace(ProfilerThreadPtr);

		// Register the shared ptr in the thread's TLS auto-cleanup list.
		// When the thread exits, it will delete the shared ptr, releasing its reference.
		(new TTlsAutoCleanupValue<FSharedPtr>(ProfilerThreadPtr))->Register();

		return ProfilerThreadPtr.Get();
	}

	static CSV_PROFILER_INLINE FCsvProfilerThreadData& Get()
	{
		FCsvProfilerThreadData* ProfilerThread = (FCsvProfilerThreadData*)FPlatformTLS::GetTlsValue(Singleton.TlsSlot);
		if (UNLIKELY(!ProfilerThread))
		{
			ProfilerThread = CreateTLSData();
		}
		return *ProfilerThread;
	}

	static CSV_PROFILER_INLINE FSharedPtr GetEndOfPipe()
	{
		static FSharedPtr EndOfPipeData = MakeShared<FCsvProfilerThreadData>(ECsvTimeline::EndOfPipe);
		return EndOfPipeData;
	}

	static inline void GetTlsInstances(TArray<FSharedPtr>& OutTlsInstances)
	{
		QUICK_SCOPE_CYCLE_COUNTER(CSVProfiler_ThreadData_GetTlsInstances);
		UE::TScopeLock Lock(Singleton.TlsCS);
		OutTlsInstances.Empty(Singleton.TlsInstances.Num() + 1);

		for (int32 Index = Singleton.TlsInstances.Num() - 1; Index >= 0; --Index)
		{
			FSharedPtr SharedPtr = Singleton.TlsInstances[Index].Pin();
			if (SharedPtr.IsValid())
			{
				// Thread is still alive.
				OutTlsInstances.Emplace(MoveTemp(SharedPtr));
			}
		}

		// Always add the end-of-pipe data
		OutTlsInstances.Add(FCsvProfilerThreadData::GetEndOfPipe());
	}

	FCsvProfilerThreadData(TOptional<ECsvTimeline::Type> InCsvTimeline = NullOpt)
		: ThreadId(FPlatformTLS::GetCurrentThreadId())
		, ThreadName(FThreadManager::GetThreadName(ThreadId))
		, bIsTaskWorkerThread(LowLevelTasks::FScheduler::Get().IsWorkerThread())
		, CsvTimeline(InCsvTimeline)
		, DataProcessor(nullptr)
	{
	}

	~FCsvProfilerThreadData()
	{
		// Don't clean up TLS data once the app is exiting - containers may have already been destroyed
		if (!GIsRunning)
		{
			return;
		}

		// No thread data processors should have a reference to this TLS instance when we're being deleted.
		check(DataProcessor == nullptr);

		QUICK_SCOPE_CYCLE_COUNTER(CSVProfiler_ThreadData_Destructor);

		// Clean up dead entries in the thread data array.
		// This will remove both the current instance, and any others that have expired.
		UE::TScopeLock Lock(Singleton.TlsCS);
		for (auto Iter = Singleton.TlsInstances.CreateIterator(); Iter; ++Iter)
		{
			if (!Iter->IsValid())
			{
				Iter.RemoveCurrent();
			}
		}
	}

	void FlushResults(TArray<FCsvTimingMarker>& OutMarkers, TArray<FCsvCustomStat>& OutCustomStats, TArray<FCsvEvent>& OutEvents)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FCsvProfilerThreadData_FlushResults);
		
		check(IsInCsvProcessingThread());

		int64 MaxSlackMemBytes = (int64)CVarMaxPerThreadStatDataSlackKB.GetValueOnAnyThread() * 1024;

		TimingMarkers.PopAll(OutMarkers, MaxSlackMemBytes);
		CustomStats.PopAll(OutCustomStats, MaxSlackMemBytes);
		Events.PopAll(OutEvents, MaxSlackMemBytes);
	}

	CSV_PROFILER_INLINE void AddTimestampBegin(const char* StatName, int32 CategoryIndex)
	{
		uint64 Cycles = FPlatformTime::Cycles64();
		TRACE_CSV_PROFILER_BEGIN_STAT(StatName, CategoryIndex, Cycles);
		TimingMarkers.ReserveElement()->Init(GetStatID(StatName), CategoryIndex, FCsvStatBase::FFlags::TimestampBegin, Cycles);
		TimingMarkers.CommitElement();
	}

	CSV_PROFILER_INLINE void AddTimestampEnd(const char* StatName, int32 CategoryIndex)
	{
		uint64 Cycles = FPlatformTime::Cycles64();
		TRACE_CSV_PROFILER_END_STAT(StatName, CategoryIndex, Cycles);
		TimingMarkers.ReserveElement()->Init(GetStatID(StatName), CategoryIndex, 0, Cycles);
		TimingMarkers.CommitElement();
	}

	CSV_PROFILER_INLINE void AddTimestampExclusiveBegin(const char* StatName)
	{
		uint64 Cycles = FPlatformTime::Cycles64();
		TRACE_CSV_PROFILER_BEGIN_EXCLUSIVE_STAT(StatName, CSV_CATEGORY_INDEX(Exclusive), Cycles);
		TimingMarkers.ReserveElement()->Init(GetStatID(StatName), CSV_CATEGORY_INDEX(Exclusive), FCsvStatBase::FFlags::TimestampBegin | FCsvStatBase::FFlags::IsExclusiveTimestamp, Cycles);
		TimingMarkers.CommitElement();
	}

	CSV_PROFILER_INLINE void AddTimestampExclusiveEnd(const char* StatName)
	{
		uint64 Cycles = FPlatformTime::Cycles64();
		TRACE_CSV_PROFILER_END_EXCLUSIVE_STAT(StatName, CSV_CATEGORY_INDEX(Exclusive), Cycles);
		TimingMarkers.ReserveElement()->Init(GetStatID(StatName), CSV_CATEGORY_INDEX(Exclusive), FCsvStatBase::FFlags::IsExclusiveTimestamp, Cycles);
		TimingMarkers.CommitElement();
	}

	CSV_PROFILER_INLINE void AddTimestampBegin(const FName& StatName, int32 CategoryIndex)
	{
		uint64 Cycles = FPlatformTime::Cycles64();
		TRACE_CSV_PROFILER_BEGIN_STAT(StatName, CategoryIndex, Cycles);
		TimingMarkers.ReserveElement()->Init(GetStatID(StatName), CategoryIndex, FCsvStatBase::FFlags::StatIDIsFName | FCsvStatBase::FFlags::TimestampBegin, Cycles);
		TimingMarkers.CommitElement();
	}

	CSV_PROFILER_INLINE void AddTimestampEnd(const FName& StatName, int32 CategoryIndex)
	{
		uint64 Cycles = FPlatformTime::Cycles64();
		TRACE_CSV_PROFILER_END_STAT(StatName, CategoryIndex, Cycles);
		TimingMarkers.ReserveElement()->Init(GetStatID(StatName), CategoryIndex, FCsvStatBase::FFlags::StatIDIsFName, Cycles);
		TimingMarkers.CommitElement();
	}

	CSV_PROFILER_INLINE void AddCustomStat(const char* StatName, const int32 CategoryIndex, const float Value, const ECsvCustomStatOp CustomStatOp)
	{
		FCsvCustomStat* CustomStat = CustomStats.ReserveElement();
		uint64 Cycles = FPlatformTime::Cycles64();
		TRACE_CSV_PROFILER_CUSTOM_STAT(StatName, CategoryIndex, Value, uint8(CustomStatOp), Cycles);
		CustomStat->Init(GetStatID(StatName), CategoryIndex, FCsvStatBase::FFlags::IsCustomStat, Cycles, uint8(CustomStatOp));
		CustomStat->Value.AsFloat = Value;
		CustomStats.CommitElement();
	}

	CSV_PROFILER_INLINE void AddCustomStat(const FName& StatName, const int32 CategoryIndex, const float Value, const ECsvCustomStatOp CustomStatOp)
	{
		FCsvCustomStat* CustomStat = CustomStats.ReserveElement();
		uint64 Cycles = FPlatformTime::Cycles64();
		TRACE_CSV_PROFILER_CUSTOM_STAT(StatName, CategoryIndex, Value, uint8(CustomStatOp), Cycles);
		CustomStat->Init(GetStatID(StatName), CategoryIndex, FCsvStatBase::FFlags::IsCustomStat | FCsvStatBase::FFlags::StatIDIsFName, Cycles, uint8(CustomStatOp));
		CustomStat->Value.AsFloat = Value;
		CustomStats.CommitElement();
	}

	CSV_PROFILER_INLINE void AddCustomStat(const char* StatName, const int32 CategoryIndex, const int32 Value, const ECsvCustomStatOp CustomStatOp)
	{
		FCsvCustomStat* CustomStat = CustomStats.ReserveElement();
		uint64 Cycles = FPlatformTime::Cycles64();
		TRACE_CSV_PROFILER_CUSTOM_STAT(StatName, CategoryIndex, Value, uint8(CustomStatOp), Cycles);
		CustomStat->Init(GetStatID(StatName), CategoryIndex, FCsvStatBase::FFlags::IsCustomStat | FCsvStatBase::FFlags::IsInteger, Cycles, uint8(CustomStatOp));
		CustomStat->Value.AsInt = Value;
		CustomStats.CommitElement();
	}

	CSV_PROFILER_INLINE void AddCustomStat(const FName& StatName, const int32 CategoryIndex, const int32 Value, const ECsvCustomStatOp CustomStatOp)
	{
		FCsvCustomStat* CustomStat = CustomStats.ReserveElement();
		uint64 Cycles = FPlatformTime::Cycles64();
		TRACE_CSV_PROFILER_CUSTOM_STAT(StatName, CategoryIndex, Value, uint8(CustomStatOp), Cycles);
		CustomStat->Init(GetStatID(StatName), CategoryIndex, FCsvStatBase::FFlags::IsCustomStat | FCsvStatBase::FFlags::IsInteger | FCsvStatBase::FFlags::StatIDIsFName, Cycles, uint8(CustomStatOp));
		CustomStat->Value.AsInt = Value;
		CustomStats.CommitElement();
	}

	CSV_PROFILER_INLINE void AddEvent(const FString& EventText, const int32 CategoryIndex)
	{
		FCsvEvent* Event = Events.ReserveElement();
		uint64 Cycles = FPlatformTime::Cycles64();
		TRACE_CSV_PROFILER_EVENT(*EventText, CategoryIndex, Cycles);
		Event->EventText = EventText;
		Event->Timestamp = Cycles;
		Event->CategoryIndex = CategoryIndex;
		Events.CommitElement();
	}

	CSV_PROFILER_INLINE void AddEventWithTimestamp(const FString& EventText, const int32 CategoryIndex, const uint64 Timestamp)
	{
		TRACE_CSV_PROFILER_EVENT(*EventText, CategoryIndex, Timestamp);
		FCsvEvent* Event = Events.ReserveElement();
		Event->EventText = EventText;
		Event->Timestamp = Timestamp;
		Event->CategoryIndex = CategoryIndex;
		Events.CommitElement();
	}

	inline uint64 GetAllocatedSize() const
	{
		// Note, we're missing the csv event FString sizes.
		// There is no way to get the events from the list without popping them.
		return
			((uint64)TimingMarkers.GetAllocatedSize()) +
			((uint64)CustomStats.GetAllocatedSize()) +
			((uint64)Events.GetAllocatedSize());
	}

	CSV_PROFILER_INLINE const FCsvWaitStatName& GetWaitStatName() const
	{
		return WaitStatNameStack.Num() == 0 ? GDefaultWaitStatName : WaitStatNameStack.Last();
	}

	CSV_PROFILER_INLINE void PushWaitStatName(const FCsvWaitStatName& WaitStatName)
	{
		LLM_SCOPE(ELLMTag::CsvProfiler);
		WaitStatNameStack.Push(WaitStatName);
	}
	CSV_PROFILER_INLINE TOptional<FCsvWaitStatName> PopWaitStatName()
	{
		if (WaitStatNameStack.Num() > 0)
		{
			return WaitStatNameStack.Pop();
		}
		return NullOpt;
	}
	// Raw stat data (written from the thread)
	TSingleProducerSingleConsumerList<FCsvTimingMarker, 256> TimingMarkers;
	TSingleProducerSingleConsumerList<FCsvCustomStat, 256> CustomStats;
	TSingleProducerSingleConsumerList<FCsvEvent, 32> Events;

	const uint32 ThreadId;
	const FString ThreadName;
	const bool bIsTaskWorkerThread;
	TOptional<ECsvTimeline::Type> CsvTimeline;

	class FCsvProfilerThreadDataProcessor* DataProcessor;
	TArray<FCsvWaitStatName> WaitStatNameStack;
};

FCsvProfilerThreadData::FSingleton FCsvProfilerThreadData::Singleton;

// Responsible to collating and accumulating stats for a thread, or group of threads
class FCsvThreadGroupStatProcessor
{
	FCsvStreamWriter* Writer;
	TArray<FCsvStatSeries*> StatSeriesArray;
	FCsvStatRegister StatRegister;
	FString Name;
	FCsvThreadGroupStatProcessor* AggregateStatProcessor;

public:
	FCsvThreadGroupStatProcessor(FCsvStreamWriter* InWriter, FString InName)
		: Writer(InWriter)
		, Name(InName)
		, AggregateStatProcessor(nullptr)
	{
	}

	~FCsvThreadGroupStatProcessor()
	{
		// Delete all the created stat series
		for (FCsvStatSeries* Series : StatSeriesArray)
		{
			delete Series;
		}
	}

	void SetAggregateStatProcessor(FCsvThreadGroupStatProcessor* InAggregateStatProcessor)
	{
		AggregateStatProcessor = InAggregateStatProcessor;
	}

	void FinalizeStatSeriesFrame(int64 FrameNumber)
	{
		for (FCsvStatSeries* Series : StatSeriesArray)
		{
			Series->FinalizeFrame(FrameNumber);
		}
	}

	FCsvStatSeries* FindOrCreateStatSeries(const FCsvStatBase& Stat, FCsvStatSeries::EType SeriesType, bool bIsCountStat, bool bIsAggregateSeries = false)
	{
		check(IsInCsvProcessingThread());

		FCsvAggregateStatSeries* LinkedAggregateStatSeries = nullptr;
		if ( AggregateStatProcessor )
		{
			// Find or create the linked aggregate stat series
			LinkedAggregateStatSeries = (FCsvAggregateStatSeries*)AggregateStatProcessor->FindOrCreateStatSeries(Stat, SeriesType, bIsCountStat, true );
		}

		const int32 StatIndex = StatRegister.GetUniqueIndex(Stat.RawStatID, Stat.CategoryIndex, Stat.IsFNameStat(), bIsCountStat);
		FCsvStatSeries* Series = nullptr;
		if (StatSeriesArray.Num() <= StatIndex)
		{
			int32 GrowBy = StatIndex + 1 - StatSeriesArray.Num();
			StatSeriesArray.AddZeroed(GrowBy);
		}
		if (StatSeriesArray[StatIndex] == nullptr)
		{
			if (bIsAggregateSeries)
			{
				Series = new FCsvAggregateStatSeries(SeriesType, StatIndex, Writer, StatRegister, Name);
			}
			else
			{
				Series = new FCsvStatSeries(SeriesType, StatIndex, Writer, StatRegister, Name, LinkedAggregateStatSeries);
			}
			StatSeriesArray[StatIndex] = Series;
		}
		else
		{
			Series = StatSeriesArray[StatIndex];
#if DO_CHECK
			checkf(SeriesType == Series->SeriesType, TEXT("Stat named %s was used in multiple stat types. Can't use same identifier for different stat types. Stat types are: Custom(Int), Custom(Float) and Timing"), *StatRegister.GetStatName(StatIndex));
#endif
		}
		return Series;
	}

	uint64 GetAllocatedSize() const
	{
		uint64 TotalSize = (uint64)StatSeriesArray.GetAllocatedSize();
		for (const FCsvStatSeries* Series : StatSeriesArray)
		{
			TotalSize += Series->GetAllocatedSize();
		}
		return TotalSize;
	}

};


class FCsvProfilerThreadDataProcessor
{
	FCsvProfilerThreadData::FSharedPtr ThreadData;
	FCsvStreamWriter* Writer;

	TArray<FCsvTimingMarker> MarkerStack;
	TArray<FCsvTimingMarker> ExclusiveMarkerStack;

	uint64 LastProcessedTimestamp;

	uint32 RenderThreadId;
	uint32 RHIThreadId;

	TSharedPtr<FCsvThreadGroupStatProcessor> StatProcessor;

public:
	FCsvProfilerThreadDataProcessor(FCsvProfilerThreadData::FSharedPtr InThreadData, FCsvStreamWriter* InWriter, uint32 InRenderThreadId, uint32 InRHIThreadId)
		: ThreadData(InThreadData)
		, Writer(InWriter)
		, LastProcessedTimestamp(0)
		, RenderThreadId(InRenderThreadId)
		, RHIThreadId(InRHIThreadId)
		, StatProcessor(new FCsvThreadGroupStatProcessor(InWriter, InThreadData->ThreadName))
	{
		check(ThreadData->DataProcessor == nullptr);
		ThreadData->DataProcessor = this;
	}

	~FCsvProfilerThreadDataProcessor()
	{
		check(ThreadData->DataProcessor == this);
		ThreadData->DataProcessor = nullptr;
	}

	inline uint64 GetAllocatedSize() const
	{
		return
			((uint64)MarkerStack.GetAllocatedSize()) +
			((uint64)ExclusiveMarkerStack.GetAllocatedSize()) +
			StatProcessor->GetAllocatedSize() +
			((uint64)ThreadData->GetAllocatedSize());
	}

	void Process(FCsvProcessThreadDataStats& OutStats, int32& OutMinFrameNumberProcessed, TSharedPtr<FCsvThreadGroupStatProcessor> TaskWorkerThreadGroupStatProcessor);

private:
	/** Temporary storage of data collected with every Process() call. */
	TArray<FCsvTimingMarker> ThreadMarkers;
	TArray<FCsvCustomStat> CustomStats;
	TArray<FCsvEvent> Events;
};


FCsvStreamWriter::FCsvStreamWriter(const TSharedRef<FArchive>& InOutputFile, bool bInContinuousWrites, int32 InBufferSize, int64 InNumFramesToBuffer, bool bInCompressOutput, uint32 InRenderThreadId, uint32 InRHIThreadId, bool bAggregateTaskWorkerStats)
	: Stream(InOutputFile, InBufferSize, bInCompressOutput, bInContinuousWrites)
	, NumFramesToBuffer(InNumFramesToBuffer)
	, WriteFrameIndex(-1)
	, ReadFrameIndex(-1)
	, bContinuousWrites(bInContinuousWrites)
	, bFirstRow(true)
	, TaskWorkerThreadGroupStatProcessor(bAggregateTaskWorkerStats ? new FCsvThreadGroupStatProcessor(this, TEXT("AllWorkers")) : nullptr)
	, RenderThreadId(InRenderThreadId)
	, RHIThreadId(InRHIThreadId)
{
}

FCsvStreamWriter::~FCsvStreamWriter()
{
	// Delete all the thread data processors, freeing all memory associated with the CSV profile
	for (FCsvProfilerThreadDataProcessor* DataProcessor : DataProcessors)
	{
		delete DataProcessor;
	}
}

void FCsvStreamWriter::AddSeries(FCsvStatSeries* Series, bool bIsVisible)
{
	check(Series->ColumnIndex == -1);
	if (bIsVisible)
	{
		Series->ColumnIndex = VisibleSeries.Num();
		VisibleSeries.Add(Series);
	}
	AllSeries.Add(Series);
}

void FCsvStreamWriter::PushValue(FCsvStatSeries* Series, int64 FrameNumber, const FCsvStatSeriesValue& Value)
{
	check(Series->ColumnIndex != -1);
	
	if (Stream.HasReachedMaxFileBytes())
	{
		return;
	}

	WriteFrameIndex = FMath::Max(FrameNumber, WriteFrameIndex);

	FCsvRow& Row = Rows.FindOrAdd(FrameNumber);

	// Ensure the row is large enough to hold every series
	if (Row.Values.Num() < VisibleSeries.Num())
	{
		Row.Values.SetNumZeroed(VisibleSeries.Num(), EAllowShrinking::No);
	}

	Row.Values[Series->ColumnIndex] = Value;
}

void FCsvStreamWriter::PushEvent(const FCsvProcessedEvent& Event)
{
	if (Stream.HasReachedMaxFileBytes())
	{
		return;
	}

	Rows.FindOrAdd(Event.FrameNumber).Events.Add(Event);
}

void FCsvStreamWriter::FinalizeNextRow()
{
	ReadFrameIndex++;

	if (bFirstRow)
	{
		// Write the first header row
		Stream.WriteString("EVENTS");

		for (FCsvStatSeries* Series : VisibleSeries)
		{
			Stream.WriteString(Series->Name);
		}

		Stream.NewLine();
		bFirstRow = false;
	}

	// Don't remove yet. Flushing series may modify this row
	FCsvRow* Row = Rows.Find(ReadFrameIndex);
	if (Row)
	{
		if (Row->Events.Num() > 0)
		{
			// Write the events for this row
			TArray<FString> EventStrings;
			EventStrings.Reserve(Row->Events.Num());
			for (FCsvProcessedEvent& Event : Row->Events)
			{
				EventStrings.Add(Event.GetFullText());
			}

			Stream.WriteSemicolonSeparatedStringList(EventStrings);
		}
		else
		{
			// No events. Insert empty string at the start of the line
			Stream.WriteEmptyString();
		}

		// Finalize the series in dependency order (non-aggregate series followed by task worker aggregate series)
		for (FCsvStatSeries* Series : AllSeries)
		{
			if (!Series->IsAggregateSeries())
			{
				Series->FinalizeFrame(ReadFrameIndex);
			}
		}
		if ( TaskWorkerThreadGroupStatProcessor )
		{	
			TaskWorkerThreadGroupStatProcessor->FinalizeStatSeriesFrame(ReadFrameIndex);
		}

		// Write the visible series to the CSV
		for (FCsvStatSeries* Series : VisibleSeries)
		{
			if (Row->Values.IsValidIndex(Series->ColumnIndex))
			{
				const FCsvStatSeriesValue& Value = Row->Values[Series->ColumnIndex];
				if (Series->SeriesType == FCsvStatSeries::EType::CustomStatInt)
				{
					Stream.WriteValue(Value.Value.AsInt);
				}
				else
				{
					Stream.WriteValue(Value.Value.AsFloat);
				}
			}
			else
			{
				Stream.WriteValue(0);
			}
		}

		Stream.NewLine();

		// Finally remove the frame data
		Rows.FindAndRemoveChecked(ReadFrameIndex);
	}
}

void FCsvStreamWriter::Finalize(const TMap<FString, FString>& Metadata)
{
	// Flush all remaining data
	while (ReadFrameIndex < WriteFrameIndex)
	{
		FinalizeNextRow();
	}

	// Write a final summary header row
	Stream.WriteString("EVENTS");
	for (FCsvStatSeries* Series : VisibleSeries)
	{
		Stream.WriteString(Series->Name);
	}
	Stream.NewLine();

	// Insert some metadata to indicate the file has a summary header row
	Stream.WriteMetadataEntry((TEXT("HasHeaderRowAtEnd")), TEXT("1"));
	Stream.WriteMetadataEntry((TEXT("EventTimestamps")), GCsvEventTimestamps ? TEXT("1") : TEXT("0") );

	// Add metadata at the end of the file, making sure commandline is last (this is required for parsing)
	const TPair<FString, FString>* CommandlineEntry = NULL;
	for (const auto& Pair : Metadata)
	{
		if (Pair.Key == "Commandline")
		{
			CommandlineEntry = &Pair;
		}
		else
		{
			Stream.WriteMetadataEntry(Pair.Key, Pair.Value);
		}
	}
	if (CommandlineEntry)
	{
		Stream.WriteMetadataEntry(CommandlineEntry->Key, CommandlineEntry->Value);
	}
}

void FCsvStreamWriter::Process(FCsvProcessThreadDataStats& OutStats)
{
	TArray<FCsvProfilerThreadData::FSharedPtr> TlsData;
	FCsvProfilerThreadData::GetTlsInstances(TlsData);

	{
		QUICK_SCOPE_CYCLE_COUNTER(CSVProfiler_Writer_GetDataProcessors);
		for (FCsvProfilerThreadData::FSharedPtr Data : TlsData)
		{
			if (!Data->DataProcessor)
			{
				DataProcessors.Add(new FCsvProfilerThreadDataProcessor(Data, this, RenderThreadId, RHIThreadId));
			}
		}
	}
	// Read the peak mem use, if configured
	OutStats.MemoryUseBytes = 0;
	if (CVarCsvTrackMemoryUse.GetValueOnAnyThread(true))
	{
		CSV_SCOPED_TIMING_STAT(CsvProfiler, GetMemoryUse);
		OutStats.MemoryUseBytes = GetAllocatedSize();
	}	

	int32 MinFrameNumberProcessed = MAX_int32;
	{
		QUICK_SCOPE_CYCLE_COUNTER(CSVProfiler_Writer_ProcessDataProcessors);
		for (FCsvProfilerThreadDataProcessor* DataProcessor : DataProcessors)
		{
			DataProcessor->Process(OutStats, MinFrameNumberProcessed, TaskWorkerThreadGroupStatProcessor);
		}
	}
	

	if (bContinuousWrites && MinFrameNumberProcessed < MAX_int32)
	{
		QUICK_SCOPE_CYCLE_COUNTER(CSVProfiler_Writer_FinalizeNextRow);
		int64 NewReadFrameIndex = MinFrameNumberProcessed - NumFramesToBuffer;
		while (ReadFrameIndex < NewReadFrameIndex)
		{
			FinalizeNextRow();
		}
	}
}

uint64 FCsvStreamWriter::GetAllocatedSize() const
{
	uint64 Size =
		((uint64)Rows.GetAllocatedSize()) +
		((uint64)AllSeries.GetAllocatedSize()) +
		((uint64)VisibleSeries.GetAllocatedSize()) +
		((uint64)DataProcessors.GetAllocatedSize()) +
		((uint64)Stream.GetAllocatedSize());

	if ( TaskWorkerThreadGroupStatProcessor.IsValid() )
	{
		Size += TaskWorkerThreadGroupStatProcessor->GetAllocatedSize();
	}

	for (const auto& Pair          : Rows)           { Size += (uint64)Pair.Value.GetAllocatedSize();     }
	for (const auto& Series        : AllSeries)      { Size += (uint64)Series->GetAllocatedSize();        }
	for (const auto& DataProcessor : DataProcessors) { Size += (uint64)DataProcessor->GetAllocatedSize(); }

	return Size;
}

inline uint64 FCsvStreamWriter::GetNumBytesWritten() const
{
	return Stream.GetNumBytesWritten();
}

inline bool FCsvStreamWriter::HasReachedMaxFileBytes() const
{
	return Stream.HasReachedMaxFileBytes();
}


struct FProcessingThreadStats
{
	double LastUpdateStartTime = 0.0;
	double LastUpdateEndTime = 0.0;
	FCsvProcessThreadDataStats ProcessingStats;
};

//-----------------------------------------------------------------------------
//	FCsvProfilerProcessingThread class : low priority thread to process 
//  profiling data
//-----------------------------------------------------------------------------
class FCsvProfilerProcessingThread : public FRunnable
{
	FThreadSafeCounter StopCounter;
	EThreadPriority CurrentThreadPriority;

	FProcessingThreadStats UpdateStats;
	FCriticalSection StatsCs;
public:
	FCsvProfilerProcessingThread(FCsvProfiler& InCsvProfiler)
		: CsvProfiler(InCsvProfiler)
	{
		uint64 AffinityMask = FPlatformAffinity::GetTaskGraphBackgroundTaskMask(); 
#if CSV_PROFILER_ALLOW_DEBUG_FEATURES
		if (FParse::Param(FCommandLine::Get(), TEXT("csvProfilerHighPriority")))
		{
			GCsvThreadPriorityIndex = 5;
			AffinityMask = FPlatformAffinity::GetTaskGraphThreadMask();
		}
#endif
		UE_LOGF(LogCsvProfiler, Display, "Starting CSV processing thread with priority %ls", GetThreadPriorityNameFromIndex(GCsvThreadPriorityIndex));
		CurrentThreadPriority = GetThreadPriorityFromIndex(GCsvThreadPriorityIndex);
		Thread = FForkProcessHelper::CreateForkableThread(this, TEXT("CSVProfiler"), 0, CurrentThreadPriority, AffinityMask);
	}

	virtual ~FCsvProfilerProcessingThread()
	{
		if (Thread)
		{
			Thread->Kill(true);
			delete Thread;
			Thread = nullptr;
		}
	}

	bool IsValid() const
	{
		return Thread != nullptr;
	}

	// FRunnable interface
	virtual bool Init() override
	{
		return true;
	}

	virtual uint32 Run() override
	{
		// Sleep until the csv profiler is fully initialized. On some platforms, TLS is not ready until this point
		while (GDelayProcessingThreadStart)
		{
			FPlatformProcess::SleepNoStats(0.001f);
		}
		// Set initial LastUpdateTime to the time the thread started so we can compute processing latency on capture start
		UpdateStats.LastUpdateStartTime = UpdateStats.LastUpdateEndTime = FPlatformTime::Seconds();

		GCsvProcessingThreadId = FPlatformTLS::GetCurrentThreadId();
		GGameThreadIsCsvProcessingThread = false;

		FMemory::SetupTLSCachesOnCurrentThread();

		LLM_SCOPE(ELLMTag::CsvProfiler);
		while (StopCounter.GetValue() == 0)
		{
			UpdateThreadPriorityIfNeeded();
			const float TimeBetweenUpdatesMS = CVarCsvProcessingThreadTimeBetweenUpdates.GetValueOnAnyThread();

			// Pause the processing thread if requested by cvar or if the benchmark is running (since it can interfere with timings under extreme load)
			if (CVarCsvPauseProcessingThread.GetValueOnAnyThread() || GCsvBenchmark.IsRunning())
			{
				FPlatformProcess::Sleep(5.0f / 1000.0f);
				continue;
			}
			double UpdateStartTime = FPlatformTime::Seconds();

			FCsvProcessThreadDataStats Stats;
			float ElapsedMs = CsvProfiler.ProcessStatData(&Stats);

			double UpdateEndTime = FPlatformTime::Seconds();

			if (GCsvProfilerIsWritingFile)
			{
				CsvProfiler.FinalizeCsvFile();
				CsvProfiler.FileWriteBlockingEvent->Trigger();
			}

			HandleGtStalling(Stats, ElapsedMs);

			// Update stats
			{
				UE::TScopeLock Lock(StatsCs);
				UpdateStats.LastUpdateStartTime = UpdateStartTime;
				UpdateStats.LastUpdateEndTime = UpdateEndTime;
				UpdateStats.ProcessingStats = Stats;
			}

			float SleepTimeSeconds = FMath::Max(TimeBetweenUpdatesMS - ElapsedMs, 0.0f) / 1000.0f;
			FPlatformProcess::Sleep(SleepTimeSeconds);
		}
		
		if (bHasGtStallLock)
		{
			StallGameThreadCs.Unlock();
		}

		FMemory::ClearAndDisableTLSCachesOnCurrentThread();

		return 0;
	}

	virtual void Stop() override
	{
		StopCounter.Increment();
	}

	virtual void Exit() override { }

	bool StallGameThreadIfNeeded()
	{
		check(IsInGameThread());

		// Handle stalling GT due to max update latency (too long since last update)
		const double MaxUpdateLatency = (double)CVarCsvProcessingThreadGtStallMaxUpdateLatencyMs.GetValueOnGameThread()/1000.0;
		if (MaxUpdateLatency > 0.0)
		{
			double GtTime = FPlatformTime::Seconds();
			double LastUpdateTime = GetStats().LastUpdateStartTime;
			double ElapsedSinceLastUpdate = GtTime - GetStats().LastUpdateStartTime;
			// If we're over the latency threshold, wait so the CSV profiler thread can catch up
			if (ElapsedSinceLastUpdate > MaxUpdateLatency)
			{
				bThreadPriorityBoost = true;
				CSV_EVENT(CsvProfiler, TEXT("GT Stall"));
				CSV_SCOPED_TIMING_STAT_EXCLUSIVE(CsvProfiler_Stall);
				CSV_SCOPED_TIMING_STAT(CsvProfiler, Stall);
				UE_LOGF(LogCsvProfiler, Warning, "Game thread stalled waiting for CSV processing thread to catch up!");
				// Stall until the next update
				do
				{
					FPlatformProcess::Sleep(0.f);
				} while (GetStats().LastUpdateStartTime < GtTime);
				return true;
			}
		}

		// Handle GT stalls triggered from the processing thread
		if (StallGameThreadCs.TryLock())
		{
			bThreadPriorityBoost = false;
			StallGameThreadCs.Unlock();
			return false;
		}
		CSV_SCOPED_TIMING_STAT_EXCLUSIVE(CsvProfiler_Stall);
		CSV_SCOPED_TIMING_STAT(CsvProfiler, Stall);
		UE::TScopeLock Lock(StallGameThreadCs);
		return true;
	}

	FProcessingThreadStats GetStats()
	{
		UE::TScopeLock Lock(StatsCs);
		return UpdateStats;
	}

private:
	static EThreadPriority GetThreadPriorityFromIndex(int32 Index)
	{
		const int32 ClampedIndex = FMath::Clamp(Index, 0, (int32)UE_ARRAY_COUNT(GCsvThreadPriorities) - 1);
		return GCsvThreadPriorities[ClampedIndex];
	}

	static const TCHAR* GetThreadPriorityNameFromIndex(int32 Index)
	{
		const int32 ClampedIndex = FMath::Clamp(Index, 0, (int32)UE_ARRAY_COUNT(GCsvThreadPriorityNames) - 1);
		return GCsvThreadPriorityNames[ClampedIndex];
	}
	void UpdateThreadPriorityIfNeeded()
	{
		int32 ThreadPriIndex = GCsvThreadPriorityIndex;
		if (bThreadPriorityBoost)
		{
			ThreadPriIndex = 5; // TPri_Highest
		}
		EThreadPriority DesiredThreadPriority = GetThreadPriorityFromIndex(ThreadPriIndex);
		if (DesiredThreadPriority != CurrentThreadPriority)
		{
			UE_LOGF(LogCsvProfiler, Display, "Setting CSV processing thread priority to %ls%ls", GetThreadPriorityNameFromIndex(ThreadPriIndex), bThreadPriorityBoost ? TEXT(" - Boost active") : TEXT(""));
			FPlatformProcess::SetThreadPriority(DesiredThreadPriority);
			CurrentThreadPriority = DesiredThreadPriority;
		}
	}

	void HandleGtStalling(const FCsvProcessThreadDataStats& ProcessingStats, float ElapsedMs)
	{
		check(IsInCsvProcessingThread());
		const float TimeBetweenUpdatesMs = CVarCsvProcessingThreadTimeBetweenUpdates.GetValueOnAnyThread();

		// Detect well-of-despair (stats coming in faster than they can be processed)
		uint32 NumStatEntriesProcessed = ProcessingStats.CustomStatCount + ProcessingStats.TimestampCount;
		TotalStatEntriesProcessed += (uint64)NumStatEntriesProcessed;
		TotalProcessingTime += (double)ElapsedMs / 1000.0;

		// If we've stalled the GT then check if we can resume
		if (StallCounter > 0)
		{
			StallCounter--;
			if (StallCounter == 0 || ElapsedMs < TimeBetweenUpdatesMs)
			{
				if (bHasGtStallLock)
				{
					StallGameThreadCs.Unlock();
					bHasGtStallLock = false;
				}
				StallCounter = 0;
			}
		}
		else 
		{
			const float StallThresholdMs = CVarCsvProcessingThreadGtStallUpdateTimeThresholdMs.GetValueOnAnyThread();
			// If this thread is slower than the last and processed more stats, increase SlowUpdateEscalationCount
			if (StallThresholdMs > 0.0f && PrevElapsedMs > StallThresholdMs && ElapsedMs > PrevElapsedMs && NumStatEntriesProcessed > PrevNumStatEntriesProcessed)
			{
				SlowUpdateEscalationCount++;
				if (SlowUpdateEscalationCount >= CVarCsvProcessingThreadGtStallUpdateEscalationThreshold.GetValueOnAnyThread())
				{
					// Stall the GT for up to 2 updates to allow us to catch up
					StallGameThreadCs.Lock();
					bHasGtStallLock = true;
					StallCounter = 2;
					double StatEntriesProcessedPerSecond = (double)TotalStatEntriesProcessed / TotalProcessingTime;
					UE_LOGF(LogCsvProfiler, Warning, "Stats coming in faster than we can process them! GT stalled until we can catch up!");
					UE_LOGF(LogCsvProfiler, Warning, "Avg processing rate: %.0f stat entries per second (timestamps+custom stats)", StatEntriesProcessedPerSecond);
					UE_LOGF(LogCsvProfiler, Warning, "Check CsvProfiler/* stats to see current rates. Run with -csvStatCounts to report per-stat counts");
					// Temporarily boost thread priority since the GT is waiting
					bThreadPriorityBoost = true;
					SlowUpdateEscalationCount = 0;
				}
			}
			else
			{
				SlowUpdateEscalationCount = 0;
			}
		}

		PrevNumStatEntriesProcessed = NumStatEntriesProcessed;
		PrevElapsedMs = ElapsedMs;

		UpdateThreadPriorityIfNeeded();
	}


	FRunnableThread* Thread;
	FCsvProfiler& CsvProfiler;

	// GT stalling
	FCriticalSection StallGameThreadCs;
	uint32 StallCounter = 0;
	double TotalProcessingTime = 0.0;
	uint64 TotalStatEntriesProcessed = 0;
	float PrevElapsedMs = 0.0f;
	uint32 PrevNumStatEntriesProcessed = 0;
	int32 SlowUpdateEscalationCount = 0;
	bool bHasGtStallLock = false;
	volatile bool bThreadPriorityBoost = false;
};

void FCsvProfilerThreadDataProcessor::Process(FCsvProcessThreadDataStats& OutStats, int32& OutMinFrameNumberProcessed, TSharedPtr<FCsvThreadGroupStatProcessor> TaskWorkerThreadGroupStatProcessor)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FCsvProfilerThreadData_ProcessThreadData);

	// We can call this from the game thread just before reading back the data, or from the CSV processing thread
	check(IsInCsvProcessingThread());

	// Read the raw CSV data
	ThreadMarkers.Reset(0);
	CustomStats.Reset(0);
	Events.Reset(0);
	ThreadData->FlushResults(ThreadMarkers, CustomStats, Events);

	FCsvThreadGroupStatProcessor* ThreadStatProcessor = StatProcessor.Get();

	// If we're aggregating task threads then link this thread's stat processor with the shared aggregate one
	FCsvThreadGroupStatProcessor* AggregateStatProcessor = nullptr;
	if (ThreadData->bIsTaskWorkerThread && TaskWorkerThreadGroupStatProcessor.IsValid())
	{
		AggregateStatProcessor = TaskWorkerThreadGroupStatProcessor.Get();
	}
	ThreadStatProcessor->SetAggregateStatProcessor(AggregateStatProcessor);

	OutStats.TimestampCount += ThreadMarkers.Num();
	OutStats.CustomStatCount += CustomStats.Num();
	OutStats.EventCount += Events.Num();

	// Flush the frame boundaries after the stat data. This way, we ensure the frame boundary data is up to date
	// (we do not want to encounter markers from a frame which hasn't been registered yet)
	FPlatformMisc::MemoryBarrier();
	ECsvTimeline::Type Timeline = ThreadData->CsvTimeline.IsSet() ? *ThreadData->CsvTimeline : ((ThreadData->ThreadId == RenderThreadId || ThreadData->ThreadId == RHIThreadId) ? ECsvTimeline::Renderthread : ECsvTimeline::Gamethread);

	if (ThreadMarkers.Num() > 0)
	{
#if !UE_BUILD_SHIPPING
		ensure(ThreadMarkers[0].GetTimestamp() >= LastProcessedTimestamp);
#endif
		LastProcessedTimestamp = ThreadMarkers.Last().GetTimestamp();
	}

	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FCsvProfilerThreadData_TimingMarkers);

		// Process timing markers
		FCsvTimingMarker InsertedMarker;
		bool bAllowExclusiveMarkerInsertion = true;
		for (int i = 0; i < ThreadMarkers.Num(); i++)
		{
			FCsvTimingMarker* MarkerPtr = &ThreadMarkers[i];

			// Handle exclusive markers. This may insert an additional marker before this one
			bool bInsertExtraMarker = false;
			if (bAllowExclusiveMarkerInsertion && MarkerPtr->IsExclusiveMarker())
			{
				if (MarkerPtr->IsBeginMarker())
				{
					if (ExclusiveMarkerStack.Num() > 0)
					{
						// Insert an artificial end marker to end the previous marker on the stack at the same timestamp
						InsertedMarker = ExclusiveMarkerStack.Last();
						InsertedMarker.Flags &= (~FCsvStatBase::FFlags::TimestampBegin);
						InsertedMarker.Flags |= FCsvStatBase::FFlags::IsExclusiveInsertedMarker;
						InsertedMarker.Timestamp = MarkerPtr->Timestamp;

						bInsertExtraMarker = true;
					}
					ExclusiveMarkerStack.Add(*MarkerPtr);
				}
				else
				{
					if (ExclusiveMarkerStack.Num() > 0)
					{
						ExclusiveMarkerStack.Pop(EAllowShrinking::No);
						if (ExclusiveMarkerStack.Num() > 0)
						{
							// Insert an artificial begin marker to resume the marker on the stack at the same timestamp
							InsertedMarker = ExclusiveMarkerStack.Last();
							InsertedMarker.Flags |= FCsvStatBase::FFlags::TimestampBegin;
							InsertedMarker.Flags |= FCsvStatBase::FFlags::IsExclusiveInsertedMarker;
							InsertedMarker.Timestamp = MarkerPtr->Timestamp;

							bInsertExtraMarker = true;
						}
					}
				}
			}

			if (bInsertExtraMarker)
			{
				// Insert an extra exclusive marker this iteration and decrement the loop index.
				MarkerPtr = &InsertedMarker;
				i--;
			}
			// Prevent a marker being inserted on the next run if we just inserted one
			bAllowExclusiveMarkerInsertion = !bInsertExtraMarker;

			FCsvTimingMarker& Marker = *MarkerPtr;
			int32 FrameNumber = GFrameBoundaries.GetFrameNumberForTimestamp(Timeline, Marker.GetTimestamp());
			OutMinFrameNumberProcessed = FMath::Min(FrameNumber, OutMinFrameNumberProcessed);
			if (Marker.IsBeginMarker())
			{
				MarkerStack.Push(Marker);
			}
			else
			{
				// Markers might not match up if they were truncated mid-frame, so we need to be robust to that
				if (MarkerStack.Num() > 0)
				{
					// Find the start marker (might not actually be top of the stack, e.g if begin/end for two overlapping stats are independent)
					bool bFoundStart = false;
#if REPAIR_MARKER_STACKS
					FCsvTimingMarker StartMarker;
					// Prevent spurious MSVC warning about this being used uninitialized further down. Alternative is to implement a ctor, but that would add overhead
					StartMarker.Init(0, 0, 0, 0);

					for (int j = MarkerStack.Num() - 1; j >= 0; j--)
					{
						if (MarkerStack[j].RawStatID == Marker.RawStatID) // Note: only works with scopes!
						{
							StartMarker = MarkerStack[j];
							MarkerStack.RemoveAt(j, EAllowShrinking::No);
							bFoundStart = true;
							break;
						}
					}
#else
					FCsvTimingMarker StartMarker = MarkerStack.Pop();
					bFoundStart = true;
#endif
					// TODO: if bFoundStart is false, this stat _never_ gets processed. Could we add it to a persistent list so it's considered next time?
					// Example where this could go wrong: staggered/overlapping exclusive stats ( e.g Abegin, Bbegin, AEnd, BEnd ), where processing ends after AEnd
					// AEnd would be missing 
					if (FrameNumber >= 0 && bFoundStart)
					{
#if !UE_BUILD_SHIPPING
						ensure(Marker.RawStatID == StartMarker.RawStatID);
						ensure(Marker.GetTimestamp() >= StartMarker.GetTimestamp());
#endif
						if (Marker.GetTimestamp() > StartMarker.GetTimestamp())
						{
							uint64 ElapsedCycles = Marker.GetTimestamp() - StartMarker.GetTimestamp();

							// Add the elapsed time to the table entry for this frame/stat
							FCsvStatSeries* Series = ThreadStatProcessor->FindOrCreateStatSeries(Marker, FCsvStatSeries::EType::TimerData, false);
							Series->SetTimerValue(FrameNumber, ElapsedCycles);

							// Add the COUNT/ series if enabled. Ignore artificial markers (inserted above)
							if (GCsvStatCounts && !Marker.IsExclusiveArtificialMarker())
							{
								FCsvStatSeries* CountSeries = ThreadStatProcessor->FindOrCreateStatSeries(Marker, FCsvStatSeries::EType::CustomStatInt, true);
								CountSeries->SetCustomStatValue_Int(FrameNumber, ECsvCustomStatOp::Accumulate, 1);
							}
						}
					}
				}
			}
		}
	}

	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FCsvProfilerThreadData_CustomStats);
		// Process the custom stats
		for (int i = 0; i < CustomStats.Num(); i++)
		{
			FCsvCustomStat& CustomStat = CustomStats[i];
			int32 FrameNumber = GFrameBoundaries.GetFrameNumberForTimestamp(Timeline, CustomStat.GetTimestamp());
			OutMinFrameNumberProcessed = FMath::Min(FrameNumber, OutMinFrameNumberProcessed);
			if (FrameNumber >= 0)
			{
				bool bIsInteger = CustomStat.IsInteger();
				FCsvStatSeries* Series = ThreadStatProcessor->FindOrCreateStatSeries(CustomStat, bIsInteger ? FCsvStatSeries::EType::CustomStatInt : FCsvStatSeries::EType::CustomStatFloat, false);
				if (bIsInteger)
				{
					Series->SetCustomStatValue_Int(FrameNumber, CustomStat.GetCustomStatOp(), CustomStat.Value.AsInt);
				}
				else
				{
					Series->SetCustomStatValue_Float(FrameNumber, CustomStat.GetCustomStatOp(), CustomStat.Value.AsFloat);
				}

				// Add the COUNT/ series if enabled
				if (GCsvStatCounts)
				{
					FCsvStatSeries* CountSeries = ThreadStatProcessor->FindOrCreateStatSeries(CustomStat, FCsvStatSeries::EType::CustomStatInt, true);
					CountSeries->SetCustomStatValue_Int(FrameNumber, ECsvCustomStatOp::Accumulate, 1);
				}
			}
		}
	}

	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FCsvProfilerThreadData_Events);

		// Process Events
		for (int i = 0; i < Events.Num(); i++)
		{
			FCsvEvent& Event = Events[i];
			int32 FrameNumber = GFrameBoundaries.GetFrameNumberForTimestamp(Timeline, Event.Timestamp);
			OutMinFrameNumberProcessed = FMath::Min(FrameNumber, OutMinFrameNumberProcessed);
			if (FrameNumber >= 0)
			{
				FCsvProcessedEvent ProcessedEvent;
				ProcessedEvent.EventText = Event.EventText;
				ProcessedEvent.FrameNumber = FrameNumber;
				ProcessedEvent.CategoryIndex = Event.CategoryIndex;
				ProcessedEvent.TimeOffsetSeconds = GCsvEventTimestamps ? FPlatformTime::ToSeconds64(Event.Timestamp-GCaptureStartTimestamp) : std::numeric_limits<double>::max();
				Writer->PushEvent(ProcessedEvent);
			}
		}
	}
}


FCsvProfiler* FCsvProfiler::Get()
{
	static FCsvProfiler* InstancePtr;

	if (!InstancePtr)
	{
		// It's important that the initializer goes here to avoid the overhead of
		// "magic static" initialization on every call (mostly an issue with MSVC
		// because of their epoch-based initialization scheme which doesn't seem
		// to make any real sense on x86)

		static FCsvProfiler Instance;
		InstancePtr = &Instance;
	}

	return InstancePtr;
}

FCsvProfiler::FCsvProfiler()
	: NumFramesToCapture(-1)
	, CaptureFrameNumber(0)
	, CaptureFrameNumberRT(0)
	, CaptureOnEventFrameCount(-1)
	, CsvGUID(FGuid(0, 0, 0, 0))
	, bInsertEndFrameAtFrameStart(false)
	, bNamedEventsWasEnabled(false)
	, LastEndFrameTimestamp(0)
	, CaptureEndFrameCount(0)
	, CaptureStartTime(0.0)
	, ProcessingThread(nullptr)
	, FileWriteBlockingEvent(FPlatformProcess::GetSynchEventFromPool())
{
	check(IsInGameThread());

#if !CSV_PROFILER_USE_CUSTOM_FRAME_TIMINGS
	FCoreDelegates::OnBeginFrame.AddStatic(CsvProfilerBeginFrame);
	FCoreDelegates::OnEndFrame.AddStatic(CsvProfilerEndFrame);
	FCoreDelegates::OnBeginFrameRT.AddStatic(CsvProfilerBeginFrameRT);
	FCoreDelegates::OnEndFrameRT.AddStatic(CsvProfilerEndFrameRT);
#endif

	// add constant metadata
	FString PlatformStr = FString::Printf(TEXT("%s"), ANSI_TO_TCHAR(FPlatformProperties::IniPlatformName()));
	FString BuildConfigurationStr = LexToString(FApp::GetBuildConfiguration());
	FString CommandlineStr = FString("\"") + FCommandLine::Get() + FString("\"");
	// Strip newlines
	CommandlineStr.ReplaceInline(TEXT("\n"), TEXT(""));
	CommandlineStr.ReplaceInline(TEXT("\r"), TEXT(""));
	FString BuildVersionString = FApp::GetBuildVersion();
	FString EngineVersionString = FEngineVersion::Current().ToString();
	FString EngineReleaseVersionString = FEngineVersion::Current().ToString(EVersionComponent::Patch);

	FString OSMajor, OSMinor;
	FPlatformMisc::GetOSVersions(OSMajor, OSMinor);
	OSMajor.TrimStartAndEndInline();
	OSMinor.TrimStartAndEndInline();
	FString OSString = FString::Printf(TEXT("%s %s"), *OSMajor, *OSMinor);

	GCsvStatNameValidator = new FCsvStatNameValidator();

	SetMetadataInternal(TEXT("Platform"), *PlatformStr);
	SetMetadataInternal(TEXT("Config"), *BuildConfigurationStr);
	SetMetadataInternal(TEXT("BuildVersion"), *BuildVersionString);
	SetMetadataInternal(TEXT("EngineVersion"), *EngineVersionString);
	SetMetadataInternal(TEXT("EngineReleaseVersion"), *EngineReleaseVersionString);
	SetMetadataInternal(TEXT("OS"), *OSString);
	SetMetadataInternal(TEXT("CPU"), *FPlatformMisc::GetDeviceMakeAndModel());
	SetMetadataInternal(TEXT("PGOEnabled"), FPlatformMisc::IsPGOEnabled() ? TEXT("1") : TEXT("0"));
	SetMetadataInternal(TEXT("PGOProfilingEnabled"), PLATFORM_COMPILER_OPTIMIZATION_PG_PROFILING ? TEXT("1") : TEXT("0"));//True if Profile Guided Optimisation Instrumentation is enabled 
	SetMetadataInternal(TEXT("LTOEnabled"), PLATFORM_COMPILER_OPTIMIZATION_LTCG ? TEXT("1") : TEXT("0"));//True if Link Time Optimisation is enabled 
	SetMetadataInternal(TEXT("ASan"), USING_ADDRESS_SANITISER ? TEXT("1") : TEXT("0"));
	SetMetadataInternal(TEXT("MergedModules"), UE_MERGED_MODULES ? TEXT("1") : TEXT("0"));
	SetMetadataInternal(TEXT("CompactTSet"), UE_USE_COMPACT_SET_AS_DEFAULT ? TEXT("1") : TEXT("0"));

	FCoreDelegates::OnSystemResolutionChanged.AddLambda([](uint32 ResX, uint32 ResY)
		{
			SetMetadata(TEXT("SystemResolution.ResX"), *LexToString(ResX));
			SetMetadata(TEXT("SystemResolution.ResY"), *LexToString(ResY));
		}
	);

#if CSV_PROFILER_ALLOW_SENSITIVE_BUILTIN_METADATA
	// for privacy, personal and free text fields are not allowed in shipping
	SetMetadataInternal(TEXT("Commandline"), *CommandlineStr, false);
	SetMetadataInternal(TEXT("LoginID"), *FPlatformMisc::GetLoginId());
	FString DeviceTag = FPlatformMisc::GetDeviceTag();
    if (!DeviceTag.IsEmpty())
    {
    	SetMetadataInternal(TEXT("DeviceTag"), *DeviceTag);
    }
	
	// Set the device ID if the platform supports it
	FString DeviceID = FPlatformMisc::GetDeviceId();
	if (!DeviceID.IsEmpty())
	{
		SetMetadataInternal(TEXT("DeviceID"), *DeviceID);
	}
#endif //CSV_PROFILER_ALLOW_SENSITIVE_BUILTIN_METADATA
}

FCsvProfiler::~FCsvProfiler()
{
	GCsvProfilerIsCapturing = false;
	IsShuttingDown.Increment();
	if (ProcessingThread)
	{
		delete ProcessingThread;
		ProcessingThread = nullptr;
	}

	if (FileWriteBlockingEvent)
	{
		FPlatformProcess::ReturnSynchEventToPool(FileWriteBlockingEvent);
		FileWriteBlockingEvent = nullptr;
	}

	if (GStartOnEvent)
	{
		delete GStartOnEvent;
		GStartOnEvent = nullptr;
	}

	if (GStopOnEvent)
	{
		delete GStopOnEvent;
		GStopOnEvent = nullptr;
	}
}

/** Per-frame update */
void FCsvProfiler::BeginFrame()
{
	LLM_SCOPE(ELLMTag::CsvProfiler);
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FCsvProfiler_BeginFrame);
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(CsvProfiler);

	check(IsInGameThread());

	// Set the thread-local waits enabled flag
	GCsvThreadLocalWaitsEnabled = GCsvTrackWaitsOnGameThread;

	if (bInsertEndFrameAtFrameStart)
	{
		bInsertEndFrameAtFrameStart = false;
		EndFrame();
	}

	if (!GCsvProfilerIsWritingFile)
	{
		// Process the command queue for start commands
		FCsvCaptureCommand CurrentCommand;
		if (CommandQueue.Peek(CurrentCommand) && CurrentCommand.CommandType == ECsvCommandType::Start)
		{
			CommandQueue.Dequeue(CurrentCommand);
			BeginCaptureInternal(CurrentCommand);
		}

		if (GCsvProfilerIsCapturing)
		{
			GFrameBoundaries.AddBeginFrameTimestamp(ECsvTimeline::Gamethread);

			if (CaptureFrameNumber == 0)
			{
				OnCSVProfileFirstFrameDelegate.Broadcast();
			}

			if (!bNamedEventsWasEnabled && (GCycleStatsShouldEmitNamedEvents > 0))
			{
				bNamedEventsWasEnabled = true;
#if !UE_SERVER
				// Servers with -EnableMetrics run perf collection a few times per match which includes
				// insights/framepro captures and namedevents on.
				// We don't want these server csvs to be filtered out by PRS, so
				// excluding this logic from server builds for now.
				SetMetadataInternal(TEXT("NamedEvents"), TEXT("1"));
#endif
			}
		}
	}

#if CSV_PROFILER_ALLOW_DEBUG_FEATURES
	if (GCsvTestingGT)
	{
		CSVTest();
	}

	GCsvABTest.BeginFrameUpdate(CaptureFrameNumber, GCsvProfilerIsCapturing);
#endif // CSV_PROFILER_ALLOW_DEBUG_FEATURES
}


void FCsvProfiler::BeginCaptureInternal(const FCsvCaptureCommand& CurrentCommand)
{
	if (GCsvProfilerIsCapturing)
	{
		UE_LOGF(LogCsvProfiler, Warning, "Capture start requested, but a capture was already running");
		return;
	}

	UE_LOGF(LogCsvProfiler, Display, "Capture Starting");
	if (GConfig)
	{
		// Update categories from the config. The config may have changed if there were hotfixes
		FCsvCategoryData::Get()->UpdateCategoriesFromConfig();
	}

	// signal external profiler that we are capturing
	OnCSVProfileStartDelegate.Broadcast();

	// Latch the cvars when we start a capture
	int32 BufferSize = FMath::Max(CVarCsvWriteBufferSize.GetValueOnAnyThread(), 0);
	bool bContinuousWrites = IsContinuousWriteEnabled(true);

	// Allow overriding of compression based on the "csv.CompressionMode" CVar
	bool bCompressOutput;
	switch (CVarCsvCompressionMode.GetValueOnGameThread())
	{
	case 0:
		bCompressOutput = false;
		break;

	case 1:
		bCompressOutput = (BufferSize > 0);
		break;

	default:
		bCompressOutput = EnumHasAnyFlags(CurrentCommand.Flags, ECsvProfilerFlags::CompressOutput) && (BufferSize > 0);
		break;
	}

	const TCHAR* CsvExtension = bCompressOutput ? TEXT(".csv.gz") : TEXT(".csv");

	// Determine the output path and filename based on override params
	FString DestinationFolder = CurrentCommand.DestinationFolder.IsEmpty() ? GetDefaultDirectory() : CurrentCommand.DestinationFolder + TEXT("/");
	FString Filename = CurrentCommand.Filename.IsEmpty() ? FString::Printf(TEXT("Profile(%s)"), *FDateTime::Now().ToString(TEXT("%Y%m%d_%H%M%S"))) : CurrentCommand.Filename;
	Filename = FPaths::SetExtension(Filename, CsvExtension);
	OutputFilename = DestinationFolder + Filename;

	TSharedPtr<FArchive> OutputFile = MakeShareable(IFileManager::Get().CreateFileWriter(*OutputFilename));
	if (!OutputFile)
	{
		UE_LOGF(LogCsvProfiler, Error, "Failed to create CSV file \"%ls\". Capture will not start.", *OutputFilename);
		return;
	}

	// Actually start the capture
	int64 NumFramesToBuffer = CVarCsvStreamFramesToBuffer.GetValueOnAnyThread();
	CsvWriter = new FCsvStreamWriter(OutputFile.ToSharedRef(), bContinuousWrites, BufferSize, NumFramesToBuffer, bCompressOutput, RenderThreadId, RHIThreadId, CVarCsvAggregateTaskWorkerStats.GetValueOnAnyThread());

	NumFramesToCapture = CurrentCommand.Value;
	GCsvRepeatFrameCount = NumFramesToCapture;
	CaptureFrameNumber = 0;
	CaptureFrameNumberRT = 0;

	if (EnumHasAnyFlags(CurrentCommand.Flags, ECsvProfilerFlags::BootCapture))
	{
		// If we're capturing from boot, set the start time to GStartTime 
		// This ensures the first time and boot timings are relative to GStartTime
		check(GFrameNumber==1);
		UE_LOGF(LogCsvProfiler, Log, "Doing a boot time capture. Start time offset %.2f", FPlatformTime::Seconds()-GStartTime);
		CaptureStartTime = GStartTime;
		// Note: we have to remove the platform time offset when converting between absolute seconds and clock cycles
        // Warning: this won't work on unix (see comment in the else case below). First frame timing will be inaccurate there when capturing from boot
		GCaptureStartTimestamp = FPlatformTime::SecondsToCycles64(CaptureStartTime - FPlatformTime::GetSecondsTimeOffset());
		
		// signal bInsertEndFrameAtFrameStart to insert an EndFrame() at the start of the first _real_ frame
		// We also add a FrameBeginTimestampsRT timestamp here, to create a dummy renderthread frame, to ensure the rows match up in the CSV
		GFrameBoundaries.AddBeginFrameTimestamp(ECsvTimeline::Renderthread, false);
		bInsertEndFrameAtFrameStart = true;
	}
	else
	{
		// Workaround for unix time inconsistencies. We can't convert between Cycles64 and Seconds because they can come from different clocks (see UnixPlatformTime.h)
		// Implementing FPlatformTime::GetSecondsTimeOffset() might help but we'd still get precision differences
		// Instead we'll record StartTime and CaptureStartTimestamp independently
		CaptureStartTime = FPlatformTime::Seconds();
		GCaptureStartTimestamp = FPlatformTime::Cycles64();
	}

	LastEndFrameTimestamp = GCaptureStartTimestamp;

	CurrentFlags = CurrentCommand.Flags;

	if (GCsvUseProcessingThread && ProcessingThread == nullptr)
	{
		// Lazily create the CSV processing thread
		ProcessingThread = new FCsvProfilerProcessingThread(*this);
		if (ProcessingThread->IsValid() == false)
		{
			UE_LOGF(LogCsvProfiler, Error, "CSV Processing Thread could not be created due to being in a single-thread environment ");
			delete ProcessingThread;
			ProcessingThread = nullptr;
			GCsvUseProcessingThread = false;
		}
	}

	// Set the CSV ID and mirror it to the log
	CsvGUID = FGuid::NewGuid();
	FString CsvIdString = CsvGUID.ToString();
	SetMetadataInternal(TEXT("CsvID"), *CsvIdString);
	UE_LOGF(LogCsvProfiler, Display, "Capture started. CSV ID: %ls", *CsvIdString);

	int32 TargetFPS = FPlatformMisc::GetMaxRefreshRate();
	static IConsoleVariable* CsvTargetFrameRateCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("csv.TargetFrameRateOverride"));
	static IConsoleVariable* MaxFPSCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("t.MaxFPS"));
	static IConsoleVariable* SyncIntervalCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("rhi.SyncInterval"));
	int32 CmdLineTargetFPS = TargetFPS;
	if (CsvTargetFrameRateCVar && CsvTargetFrameRateCVar->GetInt() > 0)
	{
		TargetFPS = CsvTargetFrameRateCVar->GetInt();
	}
	else if (FParse::Value(FCommandLine::Get(), TEXT("csv.TargetFrameRateOverride"), CmdLineTargetFPS)) // Too early to set CsvTargetFrameRateCVar with execcmds
	{
		TargetFPS = CmdLineTargetFPS;
	}
	else
	{
		// Figure out the target framerate
		if (MaxFPSCVar && MaxFPSCVar->GetInt() > 0)
		{
			TargetFPS = MaxFPSCVar->GetInt();
		}
		if (SyncIntervalCVar && SyncIntervalCVar->GetInt() > 0)
		{
			TargetFPS = FMath::Min(TargetFPS, FPlatformMisc::GetMaxRefreshRate() / SyncIntervalCVar->GetInt());
		}
	}

	// Reset the state of csv event triggers, because there could be some stale state from previous CSVProfile session
	// and event were triggered at the last frame but didn't have a chance to be consumed
	if (GCsvEventExecCmds)
	{
		for (FEventExecCmds& EventPair : *GCsvEventExecCmds)
		{
			FPlatformAtomics::AtomicStore(&EventPair.bIsActive, (int32)0);
		}
	}

	SetMetadataInternal(TEXT("TargetFramerate"), *FString::FromInt(TargetFPS));
	SetMetadataInternal(TEXT("StartTimestamp"), *FString::Printf(TEXT("%lld"), FDateTime::UtcNow().ToUnixTimestamp()));
	SetMetadataInternal(TEXT("NamedEvents"), (GCycleStatsShouldEmitNamedEvents > 0) ? TEXT("1") : TEXT("0"));
	SetMetadataInternal(TEXT("HasGtStalls"), TEXT("0"));

	static IConsoleVariable* VSyncCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.VSync"));
	if (VSyncCVar)
	{
		SetMetadataInternal(TEXT("VSyncEnabled"), *FString::FromInt(VSyncCVar->GetInt()));
	}

	if (FPlatformMemory::GetProgramSize() > 0)
	{
		// Some platforms adjust program size at runtime (based on DLL initialization), so with do this on start capture rather than in CsvProfiler::Init()
		SetMetadataInternal(TEXT("ProgramSizeMB"), *FString::SanitizeFloat((float)FPlatformMemory::GetProgramSize() / 1024.0f / 1024.0f));
	}

	const EDeviceThermalState ThermalState = FPlatformMisc::GetDeviceThermalState();
	if (ThermalState != EDeviceThermalState::Unsupported)
	{
		SetNonPersistentMetadata(TEXT("ThermalState_Start"), *FString::FromInt((int32)ThermalState));
	}

	bNamedEventsWasEnabled = (GCycleStatsShouldEmitNamedEvents > 0);

	GCsvStatCounts = !!CVarCsvStatCounts.GetValueOnGameThread();

	// Check TLS is initialized before starting the capture. This should have happened in BeginCapture
	check(FCsvProfilerThreadData::IsTlsSlotInitialized());
	TRACE_CSV_PROFILER_BEGIN_CAPTURE(*Filename, RenderThreadId, RHIThreadId, GDefaultWaitStatName.StatName, GCsvStatCounts);
	GCsvProfilerIsCapturing = true;

	// Run the Csv benchmark if enabled
	if (CVarCsvBenchmark.GetValueOnGameThread() || FParse::Param(FCommandLine::Get(), TEXT("csvBench")))
	{
		// This runs on the frame before the CSV kicks off, so doesn't affect timings
		GCsvBenchmark.Run(CVarCsvBenchmarkIterationCount.GetValueOnGameThread());
	}
}


void FCsvProfiler::EndFrame()
{
	LLM_SCOPE(ELLMTag::CsvProfiler);

	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(CsvProfiler);

	check(IsInGameThread());
	if (GCsvProfilerIsCapturing)
	{
		if (ProcessingThread)
		{
			FProcessingThreadStats ProcessingThreadStats = ProcessingThread->GetStats();
			double ElapsedSinceLastUpdate = FPlatformTime::Seconds()-ProcessingThreadStats.LastUpdateStartTime;

			if (ProcessingThreadStats.ProcessingStats.MemoryUseBytes>0)
			{
				CSV_CUSTOM_STAT(CsvProfiler, MemUsedLastUpdateMB, (float)ProcessingThreadStats.ProcessingStats.MemoryUseBytes / 1024.0f / 1024.0f, ECsvCustomStatOp::Set);
			}
			CSV_CUSTOM_STAT(CsvProfiler, UpdateLatencyMs, (float)ElapsedSinceLastUpdate*1000.0f, ECsvCustomStatOp::Set);

			// Stall the GT if needed
			if ( ProcessingThread->StallGameThreadIfNeeded() )
			{
				CSV_METADATA(TEXT("HasGtStalls"), TEXT("1"));
			}
		}		

		OnCSVProfileEndFrameDelegate.Broadcast();

		GCsvPersistentCustomStats.RecordStats();

		QUICK_SCOPE_CYCLE_COUNTER(STAT_FCsvProfiler_EndFrame_Capturing);
		if (NumFramesToCapture >= 0)
		{
			NumFramesToCapture--;
			if (NumFramesToCapture == 0)
			{
				EndCapture();
			}
		}

		const uint64 CurrentTimeStamp = FPlatformTime::Cycles64();

		// If we're doing a boot capture, just use the legacy frametime method, otherwise the initial EngineInit frames won't be set
		if (EnumHasAnyFlags(CurrentFlags, ECsvProfilerFlags::BootCapture) || CVarCsvUseLegacyFrameTime->GetBool())
		{
			// Record the frametime (measured since the last EndFrame)
			uint64 ElapsedCycles = CurrentTimeStamp - LastEndFrameTimestamp;
			double ElapsedMs = FPlatformTime::ToMilliseconds64(ElapsedCycles);
			CSV_CUSTOM_STAT_MINIMAL_GLOBAL(FrameTime, ElapsedMs, ECsvCustomStatOp::Set);
		}

		FPlatformMemoryStats MemoryStats = PlatformMemoryHelpers::GetFrameMemoryStats();

		float PhysicalMBFree = float(MemoryStats.AvailablePhysical) / (1024.0f * 1024.0f);
		float UsedExtendedMB = 0;
		float PhysicalMBUsed = float(MemoryStats.UsedPhysical) / (1024.0f * 1024.0f);
		float VirtualMBUsed = float(MemoryStats.UsedVirtual) / (1024.0f * 1024.0f);

		// infer the max we can allocate
		float TotalSystemMB = PhysicalMBFree + PhysicalMBUsed;
#if !UE_BUILD_SHIPPING
		// Subtract any extra development memory from physical free. This can result in negative values in cases where we would have crashed OOM
		PhysicalMBFree -= float(FPlatformMemory::GetExtraDevelopmentMemorySize() / 1024ull / 1024ull);
		UsedExtendedMB = PhysicalMBFree < 0.0f ? -PhysicalMBFree : 0;

		TotalSystemMB -= float(FPlatformMemory::GetExtraDevelopmentMemorySize() / 1024ull / 1024ull);
#endif
		
		CSV_CUSTOM_STAT_MINIMAL_GLOBAL(MemoryFreeMB, PhysicalMBFree, ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT_MINIMAL_GLOBAL(PhysicalUsedMB, PhysicalMBUsed, ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT_MINIMAL_GLOBAL(VirtualUsedMB, VirtualMBUsed, ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT_MINIMAL_GLOBAL(ExtendedUsedMB, UsedExtendedMB, ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT_MINIMAL_GLOBAL(SystemMaxMB, TotalSystemMB, ECsvCustomStatOp::Set);

		MemoryStats.SetEndFrameCsvStats();

		// If we're single-threaded, process the stat data here
		if (ProcessingThread == nullptr)
		{
			ProcessStatData();
		}

		// Note: we use the end frame timestamp even when CVarCsvUseLegacyFrameTime is false as 
		// this timestamp is the csv frame boundary and not necessarily the logical frame boundary.
		LastEndFrameTimestamp = CurrentTimeStamp;
		CaptureFrameNumber++;
	}

	// Process the command queue for stop commands
	FCsvCaptureCommand CurrentCommand;
	if (CommandQueue.Peek(CurrentCommand) && CurrentCommand.CommandType == ECsvCommandType::Stop)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FCsvProfiler_EndFrame_Stop);
		if ( TryEndCaptureInternal(CurrentCommand) )
		{
			// Pop the 'stop' command now that the capture has ended (or we weren't capturing anyway).
			CommandQueue.Dequeue(CurrentCommand);
		}
	}

	GCsvProfilerFrameNumber++;
}

bool FCsvProfiler::TryEndCaptureInternal(const FCsvCaptureCommand& CurrentCommand)
{
	if (GCsvProfilerIsCapturing || GCsvProfilerIsWritingFile)
	{
		// Delay end capture by a frame to allow RT stats to catch up
		if (CurrentCommand.FrameRequested == GCsvProfilerFrameNumber)
		{
			CaptureEndFrameCount = CaptureFrameNumber;
			return false;
		}

		UE_LOGF(LogCsvProfiler, Display, "Capture Stop requested");

		// signal external profiler that we are done
		OnCSVProfileEndDelegate.Broadcast();

		// Signal to the processing thread to write the file out (if we have one).
		GCsvProfilerIsWritingFile = true;
		GCsvProfilerIsCapturing = false;

		TRACE_CSV_PROFILER_END_CAPTURE();

		if (!ProcessingThread)
		{
			// Suspend the hang and hitch heartbeats, as this is a long running task.
			FSlowHeartBeatScope SuspendHeartBeat;
			FDisableHitchDetectorScope SuspendGameThreadHitch;

			// No processing thread, block and write the file out on the game thread. We're done.
			FinalizeCsvFile();
		}
		else
		{
			if (CVarCsvBlockOnCaptureEnd.GetValueOnGameThread() == 1)
			{
				// Suspend the hang and hitch heartbeats, as this is a long running task.
				FSlowHeartBeatScope SuspendHeartBeat;
				FDisableHitchDetectorScope SuspendGameThreadHitch;

				// Block the game thread here whilst the result file is written out.
				FileWriteBlockingEvent->Wait();
			}
			// Not done yet...
			return false;
		}
	}

	// If we get here then we're actually done
	check(!GCsvProfilerIsCapturing && !GCsvProfilerIsWritingFile);

	// Signal the async completion callback, if one was provided when the capture was stopped.
	if (CurrentCommand.Completion)
	{
		CurrentCommand.Completion->SetValue(OutputFilename);
		delete CurrentCommand.Completion;
	}

	FileWriteBlockingEvent->Reset();

	// No output filename means we weren't running a capture.
	bool bCaptureEnded = true;
	if (OutputFilename.IsEmpty())
	{
		UE_LOGF(LogCsvProfiler, Warning, "Capture Stop requested, but no capture was running!");
	}
	else
	{
		OutputFilename.Reset();

		// Handle repeats
		if (GCsvRepeatCount != 0 && GCsvRepeatFrameCount > 0)
		{
			if (GCsvRepeatCount > 0)
			{
				GCsvRepeatCount--;
			}
			if (GCsvRepeatCount != 0)
			{
				bCaptureEnded = false;

				// TODO: support directories
				BeginCapture(GCsvRepeatFrameCount);
			}
		}
	}

	if (bCaptureEnded && (GCsvExitOnCompletion || FParse::Param(FCommandLine::Get(), TEXT("ExitAfterCsvProfiling"))))
	{
		bool bForceExit = !!CVarCsvForceExit.GetValueOnGameThread();
		FPlatformMisc::RequestExit(bForceExit, TEXT("CsvProfiler.ExitAfterCsvProfiling"));
	}
	return true;
}

void FCsvProfiler::OnEndFramePostFork()
{
	// Reinitialize commandline-based configuration
	GCsvUseProcessingThread = FForkProcessHelper::IsForkedMultithreadInstance() && !FParse::Param(FCommandLine::Get(), TEXT("csvNoProcessingThread"));
	GGameThreadIsCsvProcessingThread = !GCsvUseProcessingThread;
	// Make sure no one called BeginCapture() before forking, as the runnable doesn't fully support the transition
	checkf(ProcessingThread == nullptr, TEXT("CSV profiling should not be started pre-fork"));
}

/** Per-frame update */
void FCsvProfiler::BeginFrameRT()
{
	LLM_SCOPE(ELLMTag::CsvProfiler);
	RenderThreadId = FPlatformTLS::GetCurrentThreadId();

	check(IsInRenderingThread());
	if (GCsvProfilerIsCapturing)
	{
		// Mark where the renderthread frames begin
		GFrameBoundaries.AddBeginFrameTimestamp(ECsvTimeline::Renderthread);
	}
	GCsvProfilerIsCapturingRT = GCsvProfilerIsCapturing;

#if CSV_PROFILER_ALLOW_DEBUG_FEATURES
	if (GCsvTestingRT)
	{
		CSVTest();
	}
#endif // CSV_PROFILER_ALLOW_DEBUG_FEATURES

	// Set the thread-local waits enabled flag
	GCsvThreadLocalWaitsEnabled = GCsvTrackWaitsOnRenderThread;
}

void FCsvProfiler::EndFrameRT()
{
	LLM_SCOPE(ELLMTag::CsvProfiler);
	check(IsInRenderingThread());
	if (GCsvProfilerIsCapturing)
	{
		CaptureFrameNumberRT++;
	}
}

void FCsvProfiler::BeginFrameEOP()
{
	if (GCsvProfilerIsCapturing)
	{
		GFrameBoundaries.AddBeginFrameTimestamp(ECsvTimeline::EndOfPipe);
	}
}

void FCsvProfiler::UpdateFrameTime()
{
	// If we're doing a boot capture, just use the legacy frametime method, otherwise the initial EngineInit frames won't be set
	if (!EnumHasAnyFlags(CurrentFlags, ECsvProfilerFlags::BootCapture) && !CVarCsvUseLegacyFrameTime->GetBool())
	{
		// This is called right after UpdateTimeAndHandleMaxTickRate() in the engine loop which is the point at which
		// we've finished waiting when a frame rate limit is enabled. Computing the frame time here means it does not 
		// align with the csv frame boundary, but it does align with the logical frame boundary.
		// This avoids noise in the frame time stat introduced by the wait at the start of the frame as it cannot possibly
		// predict how long this frame will be and is instead based on the time of previous frames. Therefore this wait 
		// period and any time before it logically belong to the prior frame. Using this period for the frame time results
		// in almost 0 noise in the frame time calculation when a frame rate limit is enabled, and no difference to computing
		// it at the end of the frame when running uncapped or with vsync enabled (besides the phase shift).

		static double LastTimestamp = 0.0;
		const double CurrentTimestamp = FPlatformTime::Seconds();
		if (LastTimestamp > 0.0)
		{
			double FrameTimeMs = (CurrentTimestamp - LastTimestamp) * 1000.0;
			CSV_CUSTOM_STAT_MINIMAL_GLOBAL(FrameTime, FrameTimeMs, ECsvCustomStatOp::Set);
		}
		LastTimestamp = CurrentTimestamp;
	}
}

void FCsvProfiler::BeginCapture(int InNumFramesToCapture, 
	const FString& InDestinationFolder, 
	const FString& InFilename,
	ECsvProfilerFlags InFlags)
{
	LLM_SCOPE(ELLMTag::CsvProfiler);
	check(IsInGameThread());

	// If there's already a start command in flight for this capture, warn and continue
	FCsvCaptureCommand CurrentCommand;
	if (CommandQueue.Peek(CurrentCommand) && CurrentCommand.CommandType == ECsvCommandType::Start)
	{
		UE_LOGF(LogCsvProfiler, Warning, "BeginCapture() called, but there is already a pending start command. Ignoring!");
		return;
	}

	// Lazy init TLS before starting the capture
	FCsvProfilerThreadData::InitTls();

	// Check if we actually have valid TLS data before starting the profile
	if (!FCsvProfilerThreadData::IsTlsSlotInitialized())
	{
		UE_LOGF(LogCsvProfiler, Error, "Failed to allocate TLS! Not starting the CSV capture");
	}
	else
	{
		CommandQueue.Enqueue(FCsvCaptureCommand(ECsvCommandType::Start, GCsvProfilerFrameNumber, InNumFramesToCapture, InDestinationFolder, InFilename, InFlags));
	}
}

TSharedFuture<FString> FCsvProfiler::EndCapture(FGraphEventRef EventToSignal)
{
	if (AutoRTFM::IsClosed())
	{
		// In a transaction we cannot support signalling an event.
		ensure(!EventToSignal.IsValid());

		// If we are in a transaction we only want to actually end the capture if we successfully
		// committed the transaction. It is safe to do this because the capture API doesn't end
		// the capture immediately, but at the next end-frame tick (in `EndFrame`).
		AutoRTFM::OnCommit([this]
		{
			EndCapture();
		});

		// TODO: Could we make a special shared future that would explode if used? We cannot support
		// shared future returns from within a transaction.
		return {};
	}

	LLM_SCOPE(ELLMTag::CsvProfiler);
	check(IsInGameThread());

	if (!IsCapturing())
	{
		UE_LOGF(LogCsvProfiler, Warning, "EndCapture() called, but no capture was in progress. Ignoring!");
		return {};
	}

	// If there's already a stop command in flight for this capture, warn and continue
	FCsvCaptureCommand CurrentCommand;
	if (CommandQueue.Peek(CurrentCommand) && CurrentCommand.CommandType == ECsvCommandType::Stop)
	{
		UE_LOGF(LogCsvProfiler, Warning, "EndCapture() called, but there is already a pending stop command. Ignoring!");
		return CurrentCommand.Future;
	}

	if (CommandQueue.Peek(CurrentCommand) && CurrentCommand.CommandType == ECsvCommandType::Start)
	{
		UE_LOGF(LogCsvProfiler, Warning, "EndCapture() called, but there is already a pending start command!");
	}

	// Fire before we copy the metadata so it gives other systems a chance to write any final information.
	OnCSVProfileEndRequestedDelegate.Broadcast();

	const EDeviceThermalState ThermalState = FPlatformMisc::GetDeviceThermalState();
	if (ThermalState != EDeviceThermalState::Unsupported)
	{
		SetNonPersistentMetadata(TEXT("ThermalState_End"), *FString::FromInt((int32)ThermalState));
	}

	SetNonPersistentMetadata(TEXT("EndTimestamp"), *FString::Printf(TEXT("%lld"), FDateTime::UtcNow().ToUnixTimestamp()));
	SetNonPersistentMetadata(TEXT("CaptureDuration"), *FString::SanitizeFloat(FPlatformTime::Seconds()-CaptureStartTime));

	bool bHasReachedMaxFileBytes = CsvWriter == nullptr ? false : CsvWriter->HasReachedMaxFileBytes();
	SetNonPersistentMetadata(TEXT("CsvMaxFileBytesReached"), bHasReachedMaxFileBytes ? TEXT("1") : TEXT("0"));

	TPromise<FString>* Completion = new TPromise<FString>([EventToSignal]()
	{
		if (EventToSignal)
		{
			EventToSignal->DispatchSubsequents();
		}
	});

	// Copy the metadata array for the next FinalizeCsvFile
	TMap<FString, FString> CopyMetadataMap;
	{
		UE::TScopeLock Lock(MetadataCS);
		CopyMetadataMap = MetadataMap;
		// Merge the metadata
		CopyMetadataMap.Append(MoveTemp(NonPersistentMetadataMap));
		// Clear now that the capture is finished.
		NonPersistentMetadataMap = TMap<FString, FString>();
	}
	MetadataQueue.Enqueue(MoveTemp(CopyMetadataMap));

	TSharedFuture<FString> Future = Completion->GetFuture().Share();
	CommandQueue.Enqueue(FCsvCaptureCommand(ECsvCommandType::Stop, GCsvProfilerFrameNumber, Completion, Future));

	return Future;
}

void FCsvProfiler::FinalizeCsvFile()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FCsvProfiler_FinalizeCsvFile);
	check(IsInCsvProcessingThread());

	UE_LOGF(LogCsvProfiler, Display, "Capture Ending");

	double FinalizeStartTime = FPlatformTime::Seconds();

	// Do a final process of the stat data
	ProcessStatData();

	uint64 MemoryBytesAtEndOfCapture = CsvWriter->GetAllocatedSize();
	
	// Get the queued metadata for the next csv finalize
	TMap<FString, FString> CurrentMetadata;
	MetadataQueue.Dequeue(CurrentMetadata);

	CsvWriter->Finalize(CurrentMetadata);

	delete CsvWriter;
	CsvWriter = nullptr;

	// TODO - Probably need to clear the frame boundaries after each completed CSV row
	GFrameBoundaries.Clear();

	UE_LOGF(LogCsvProfiler, Display, "Capture Ended. Writing CSV to file : %ls", *OutputFilename);
	UE_LOGF(LogCsvProfiler, Display, "  Frames : %d", CaptureEndFrameCount);
	UE_LOGF(LogCsvProfiler, Display, "  Peak memory usage  : %.2fMB", float(MemoryBytesAtEndOfCapture) / (1024.0f * 1024.0f));

	OnCSVProfileFinished().Broadcast(OutputFilename);

	float FinalizeDuration = float(FPlatformTime::Seconds() - FinalizeStartTime);
	UE_LOGF(LogCsvProfiler, Display, "  CSV finalize time : %.3f seconds", FinalizeDuration);

	GCsvProfilerIsWritingFile = false;
}

void FCsvProfiler::SetDeviceProfileName(FString InDeviceProfileName)
{
	CSV_METADATA(TEXT("DeviceProfile"), *InDeviceProfileName);
}

/** Push/pop events */
void FCsvProfiler::BeginStat(const char * StatName, uint32 CategoryIndex, const char * NamedEventName)
{
#if !CSV_PROFILER_MINIMAL
	if (GCsvProfilerIsCapturing && GCsvCategoriesEnabled[CategoryIndex])
	{
#if CSV_PROFILER_SUPPORT_NAMED_EVENTS
		if (UNLIKELY(GCsvProfilerNamedEventsTiming))
		{
			CsvBeginNamedEvent(FColor(255, 128, 255), NamedEventName ? NamedEventName : StatName);
		}
#endif
		FCsvProfilerThreadData::Get().AddTimestampBegin(StatName, CategoryIndex);
	}
#endif
}

void FCsvProfiler::BeginStat(const FName& StatName, uint32 CategoryIndex)
{
#if !CSV_PROFILER_MINIMAL
	if (GCsvProfilerIsCapturing && GCsvCategoriesEnabled[CategoryIndex])
	{
		ValidateFName(StatName);
#if CSV_PROFILER_SUPPORT_NAMED_EVENTS
		if (UNLIKELY(GCsvProfilerNamedEventsTiming))
		{
			CsvBeginNamedEvent(FColor(255, 128, 255), StatName);
		}
#endif
		FCsvProfilerThreadData::Get().AddTimestampBegin(StatName, CategoryIndex);
	}
#endif
}

void FCsvProfiler::EndStat(const char * StatName, uint32 CategoryIndex)
{
#if !CSV_PROFILER_MINIMAL
	if (GCsvProfilerIsCapturing && GCsvCategoriesEnabled[CategoryIndex])
	{
		FCsvProfilerThreadData::Get().AddTimestampEnd(StatName, CategoryIndex);
#if CSV_PROFILER_SUPPORT_NAMED_EVENTS
		if (UNLIKELY(GCsvProfilerNamedEventsTiming))
		{
			CsvEndNamedEvent();
		}
#endif
	}
#endif
}

void FCsvProfiler::EndStat(const FName& StatName, uint32 CategoryIndex)
{
#if !CSV_PROFILER_MINIMAL
	if (GCsvProfilerIsCapturing && GCsvCategoriesEnabled[CategoryIndex])
	{
		ValidateFName(StatName);
		FCsvProfilerThreadData::Get().AddTimestampEnd(StatName, CategoryIndex);
#if CSV_PROFILER_SUPPORT_NAMED_EVENTS
		if (UNLIKELY(GCsvProfilerNamedEventsTiming))
		{
			CsvEndNamedEvent();
		}
#endif
	}
#endif
}

void FCsvProfiler::BeginExclusiveStat(const char * StatName, const char * NamedEventName)
{
#if !CSV_PROFILER_MINIMAL
	if (GCsvProfilerIsCapturing && GCsvCategoriesEnabled[CSV_CATEGORY_INDEX(Exclusive)])
	{
#if CSV_PROFILER_SUPPORT_NAMED_EVENTS
		if (UNLIKELY(GCsvProfilerNamedEventsExclusive))
		{
			CsvBeginNamedEvent(FColor(255, 128, 128), NamedEventName ? NamedEventName : StatName);
		}
#endif
		FCsvProfilerThreadData::Get().AddTimestampExclusiveBegin(StatName);
	}
#endif
}

void FCsvProfiler::EndExclusiveStat(const char * StatName)
{
#if !CSV_PROFILER_MINIMAL
	if (GCsvProfilerIsCapturing && GCsvCategoriesEnabled[CSV_CATEGORY_INDEX(Exclusive)])
	{
		FCsvProfilerThreadData::Get().AddTimestampExclusiveEnd(StatName);
#if CSV_PROFILER_SUPPORT_NAMED_EVENTS
		if (UNLIKELY(GCsvProfilerNamedEventsExclusive))
		{
			CsvEndNamedEvent();
		}
#endif
	}
#endif
}


void FCsvProfiler::BeginSetWaitStat(const char* StatName, const char* FormattedStatName, const char* FormattedStatNameNonCP)
{
#if !CSV_PROFILER_MINIMAL
	if (GCsvProfilerIsCapturing && GCsvCategoriesEnabled[CSV_CATEGORY_INDEX(Exclusive)])
	{
		if (StatName == nullptr)
		{
			FCsvProfilerThreadData::Get().PushWaitStatName(GIgnoreWaitStatName);
		}
		else
		{
			FCsvProfilerThreadData::Get().PushWaitStatName(FCsvWaitStatName(StatName, FormattedStatName, FormattedStatNameNonCP));
		}
	}
#endif
}

void FCsvProfiler::EndSetWaitStat()
{
#if !CSV_PROFILER_MINIMAL
	if (GCsvProfilerIsCapturing && GCsvCategoriesEnabled[CSV_CATEGORY_INDEX(Exclusive)])
	{
		FCsvProfilerThreadData::Get().PopWaitStatName();
	}
#endif
}

void FCsvProfiler::BeginWait()
{
#if !CSV_PROFILER_MINIMAL
	if (GCsvProfilerIsCapturing && GCsvCategoriesEnabled[CSV_CATEGORY_INDEX(Exclusive)])
	{
		const FCsvWaitStatName& WaitStatName = FCsvProfilerThreadData::Get().GetWaitStatName();
		if (WaitStatName != GIgnoreWaitStatName)
		{
#if CSV_PROFILER_SUPPORT_NAMED_EVENTS
			if (UNLIKELY(GCsvProfilerNamedEventsExclusive))
			{
				if ( UE::Stats::FThreadIdleStats::Get().IsCriticalPath() )
				{
					CsvBeginNamedEvent(FColor(192, 96, 96), "CsvEventWait");

					if (WaitStatName.FormattedStatName != nullptr)
					{
						CsvBeginNamedEvent(FColor(192, 96, 96), WaitStatName.FormattedStatName);
					}
				}
				else
				{
					CsvBeginNamedEvent(FColor(255, 128, 128), "CsvEventWait (Non-CP)");

					if (WaitStatName.FormattedStatNameNonCP != nullptr)
					{
						CsvBeginNamedEvent(FColor(255, 128, 128), WaitStatName.FormattedStatNameNonCP);
					}
				}
			}
#endif
			FCsvProfilerThreadData::Get().AddTimestampExclusiveBegin(WaitStatName.StatName);
		}
	}
#endif
}

void FCsvProfiler::EndWait()
{
#if !CSV_PROFILER_MINIMAL
	if (GCsvProfilerIsCapturing && GCsvCategoriesEnabled[CSV_CATEGORY_INDEX(Exclusive)])
	{
		const FCsvWaitStatName& WaitStatName = FCsvProfilerThreadData::Get().GetWaitStatName();
		if (WaitStatName != GIgnoreWaitStatName)
		{
			FCsvProfilerThreadData::Get().AddTimestampExclusiveEnd(WaitStatName.StatName);
#if CSV_PROFILER_SUPPORT_NAMED_EVENTS
			if (UNLIKELY(GCsvProfilerNamedEventsExclusive))
			{
				CsvEndNamedEvent();

				if (WaitStatName.FormattedStatName != nullptr || WaitStatName.FormattedStatNameNonCP != nullptr)
				{
					CsvEndNamedEvent();
				}
			}
#endif
		}
	}
#endif
}

FString FCsvProfiler::GetDefaultDirectory()
{
	return FPaths::ProfilingDir() + TEXT("CSV/");
}

void FCsvProfiler::RecordEventfInternal(int32 CategoryIndex, bool bLog, const TCHAR* Fmt, ...)
{
	bool bIsCsvRecording = GCsvProfilerIsCapturing && GCsvCategoriesEnabled[CategoryIndex];
	if (bIsCsvRecording || GStartOnEvent)
	{
		LLM_SCOPE(ELLMTag::CsvProfiler);
		TCHAR Buffer[256];
		GET_TYPED_VARARGS(TCHAR, Buffer, UE_ARRAY_COUNT(Buffer), UE_ARRAY_COUNT(Buffer) - 1, Fmt, Fmt);
		Buffer[255] = '\0';
		FString Str = Buffer;

		if (bIsCsvRecording)
		{
			RecordEvent(CategoryIndex, Str, bLog);

#if CSV_PROFILER_ALLOW_DEBUG_FEATURES
			if (GStopOnEvent && GStopOnEvent->Equals(Str, ESearchCase::IgnoreCase))
			{
				FCsvProfiler::Get()->EndCapture();
			}
#endif
		}
#if CSV_PROFILER_ALLOW_DEBUG_FEATURES
		else
		{
			if (GStartOnEvent && GStartOnEvent->Equals(Str, ESearchCase::IgnoreCase))
			{
				FCsvProfiler::Get()->BeginCapture(FCsvProfiler::Get()->GetNumFrameToCaptureOnEvent());
			}
		}
#endif
	}
}

void FCsvProfiler::RecordEvent(int32 CategoryIndex, const FString& EventText, bool bLog, bool bAllowDelegate)
{
	if (GCsvProfilerIsCapturing && GCsvCategoriesEnabled[CategoryIndex])
	{
		LLM_SCOPE(ELLMTag::CsvProfiler);

		FCsvProfilerThreadData::Get().AddEvent(EventText, CategoryIndex);

		// Log the event and broadcast to delegates
		FString CategoryName;
		FString FullEventText = EventText;
		if (CategoryIndex != CSV_CATEGORY_INDEX_GLOBAL)
		{
			CategoryName = FCsvCategoryData::Get()->GetCategoryNameByIndex(CategoryIndex);
			if (bLog || GCsvEventExecCmds)
			{
				FullEventText = FString::Printf(TEXT("%s/%s"), *CategoryName, *EventText);
			}			
		}
		if (bLog)
		{
			UE_LOGF(LogCsvProfiler, Display, "CSVEvent \"%ls\" [Frame %d]", *FullEventText, FCsvProfiler::Get()->GetCaptureFrameNumber());
		}
#if CSV_PROFILER_ALLOW_DEBUG_FEATURES
		if (bAllowDelegate)
		{
			FCsvProfiler::Get()->OnCSVProfileEvent().Broadcast(CategoryName, EventText);
		}
		if (GCsvEventExecCmds)
		{
			FStringView Match(FullEventText);
			for (FEventExecCmds& EventWildcard : *GCsvEventExecCmds)
			{
				if (FWildcardString::IsMatchSubstring(*EventWildcard.EventWildcard, Match.begin(), Match.end(), ESearchCase::IgnoreCase))
				{
					FPlatformAtomics::AtomicStore(&EventWildcard.bIsActive, 1);
				}
			}
		}
#endif 
	}
}

void FCsvProfiler::RecordEventAtFrameStart(int32 CategoryIndex, const FString& EventText, bool bLog)
{
	RecordEventAtTimestamp(CategoryIndex, EventText, FCsvProfiler::Get()->LastEndFrameTimestamp, bLog);
}

void FCsvProfiler::BeginRegion(int32 CategoryIndex, const TCHAR* RegionName, const TCHAR* SubCategoryName)
{
	if (GCsvProfilerIsCapturing && GCsvCategoriesEnabled[CategoryIndex])
	{
		RecordRegionEventInternal(CategoryIndex == CSV_CATEGORY_INDEX_GLOBAL ? nullptr : *FCsvCategoryData::Get()->GetCategoryNameByIndex(CategoryIndex), RegionName, SubCategoryName, true);
	}
}

void FCsvProfiler::EndRegion(int32 CategoryIndex, const TCHAR* RegionName, const TCHAR* SubCategoryName)
{
	if (GCsvProfilerIsCapturing && GCsvCategoriesEnabled[CategoryIndex])
	{
		RecordRegionEventInternal(CategoryIndex == CSV_CATEGORY_INDEX_GLOBAL ? nullptr : *FCsvCategoryData::Get()->GetCategoryNameByIndex(CategoryIndex), RegionName, SubCategoryName, false);
	}
}

void FCsvProfiler::RecordRegionEventInternal(const TCHAR* CategoryName, const TCHAR* RegionName, const TCHAR* SubCategoryName, bool bBegin)
{
	const TCHAR* Suffix = bBegin ? TEXT("Begin") : TEXT("End");
	FString EventText;
	if (CategoryName)
	{
		EventText = SubCategoryName ?
			FString::Printf(TEXT("Region: %s/%s/%s - %s"), CategoryName, SubCategoryName, RegionName, Suffix) :
			FString::Printf(TEXT("Region: %s/%s - %s"), CategoryName, RegionName, Suffix);
	}
	else
	{
		EventText = SubCategoryName ? 
			FString::Printf(TEXT("Region: %s/%s - %s"), SubCategoryName, RegionName, Suffix) : 
			FString::Printf(TEXT("Region: %s - %s"), RegionName, Suffix);
	}
	RecordEvent(CSV_CATEGORY_INDEX_GLOBAL, EventText, false, false);
}


void FCsvProfiler::SetMetadata(const TCHAR* Key, const TCHAR* Value)
{
	FCsvProfiler::Get()->SetMetadataInternal(Key, Value, true, EMetadataPersistenceType::Persistent);
}

void FCsvProfiler::SetNonPersistentMetadata(const TCHAR* Key, const TCHAR* Value)
{
	FCsvProfiler::Get()->SetMetadataInternal(Key, Value, true, EMetadataPersistenceType::NonPersistent);
}

TMap<FString, FString> FCsvProfiler::GetMetadataMapCopy()
{
	LLM_SCOPE(ELLMTag::CsvProfiler);
	UE::TScopeLock Lock(MetadataCS);
	TMap<FString, FString> MetadataMapCopy = MetadataMap;
	MetadataMapCopy.Append(NonPersistentMetadataMap);
	return MetadataMapCopy;
}

void FCsvProfiler::SetMetadataInternal(const TCHAR* Key, const TCHAR* Value, bool bSanitize, EMetadataPersistenceType PersistenceType)
{
	// Always gather CSV metadata, even if we're not currently capturing.
	// Metadata is applied to the next CSV profile, when the file is written.
	LLM_SCOPE(ELLMTag::CsvProfiler);
	FString KeyLower = FString(Key).ToLower();

	TMap<FString, FString>& CurrentMetadataMap = PersistenceType == EMetadataPersistenceType::Persistent ? MetadataMap : NonPersistentMetadataMap;
	if (Value == nullptr)
	{
		UE::TScopeLock Lock(MetadataCS);
		if (CurrentMetadataMap.Contains(KeyLower))
		{
			UE_LOGF(LogCsvProfiler, Display, "Metadata unset : %ls", *KeyLower);
			CurrentMetadataMap.Remove(KeyLower);
		}
	}
	else
	{
		TRACE_CSV_PROFILER_METADATA(Key, Value);
		FString ValueStr = Value;
		if (bSanitize)
		{
			check(!KeyLower.Contains(TEXT(",")));
			if (ValueStr.ReplaceInline(TEXT(","), TEXT("&#44;")) > 0)
			{
				UE_LOGF(LogCsvProfiler, Warning, "Metadata value sanitized due to invalid characters: %ls=\"%ls\"", *KeyLower, Value);
			}
		}
		// Only log if the metadata changed, to prevent logspam 
		UE::TScopeLock Lock(MetadataCS);
		if (!CurrentMetadataMap.Contains(KeyLower) || CurrentMetadataMap[KeyLower] != ValueStr)
		{
			UE_LOGF(LogCsvProfiler, Display, "Metadata set : %ls=\"%ls\"", *KeyLower, *ValueStr);
		}
		CurrentMetadataMap.FindOrAdd(KeyLower) = ValueStr;
	}
}

void FCsvProfiler::RecordEventAtTimestamp(int32 CategoryIndex, const FString& EventText, uint64 Cycles64, bool bLog)
{
	if (GCsvProfilerIsCapturing && GCsvCategoriesEnabled[CategoryIndex])
	{
		LLM_SCOPE(ELLMTag::CsvProfiler);
		if (bLog)
		{
			UE_LOGF(LogCsvProfiler, Display, "CSVEvent [Frame %d] : \"%ls\"", FCsvProfiler::Get()->GetCaptureFrameNumber(), *EventText);
		}
		FCsvProfilerThreadData::Get().AddEventWithTimestamp(EventText, CategoryIndex,Cycles64);

		if (IsContinuousWriteEnabled(false))
		{
			UE_LOGF(LogCsvProfiler, Warning, 
				"RecordEventAtTimestamp is not compatible with continuous CSV writing. " "Some events may be missing in the output file. Set 'csv.ContinuousWrites' " "to 0 to ensure events recorded with specific timestamps are captured correctly.");
		}
	}
}

void FCsvProfiler::RecordCustomStatMinimal(const char* StatName, uint32 CategoryIndex, float Value, const ECsvCustomStatOp CustomStatOp)
{
	if (GCsvProfilerIsCapturing && GCsvCategoriesEnabled[CategoryIndex])
	{
		FCsvProfilerThreadData::Get().AddCustomStat(StatName, CategoryIndex, Value, CustomStatOp);
	}
}

void FCsvProfiler::RecordCustomStatMinimal(const char* StatName, uint32 CategoryIndex, int32 Value, const ECsvCustomStatOp CustomStatOp)
{
	if (GCsvProfilerIsCapturing && GCsvCategoriesEnabled[CategoryIndex])
	{
		FCsvProfilerThreadData::Get().AddCustomStat(StatName, CategoryIndex, Value, CustomStatOp);
	}
}

void FCsvProfiler::RecordCustomStatMinimal(const char* StatName, uint32 CategoryIndex, double Value, const ECsvCustomStatOp CustomStatOp)
{
	// LWC_TODO: Double support for FCsvProfiler::RecordCustomStat
	RecordCustomStatMinimal(StatName, CategoryIndex, (float)Value, CustomStatOp);
}


void FCsvProfiler::RecordCustomStat(const char * StatName, uint32 CategoryIndex, float Value, const ECsvCustomStatOp CustomStatOp)
{
#if !CSV_PROFILER_MINIMAL
	if (GCsvProfilerIsCapturing && GCsvCategoriesEnabled[CategoryIndex])
	{
		UE_AUTORTFM_OPEN
		{
			FCsvProfilerThreadData::Get().AddCustomStat(StatName, CategoryIndex, Value, CustomStatOp);
		};
	}
#endif
}

void FCsvProfiler::RecordCustomStat(const char* StatName, uint32 CategoryIndex, double Value, const ECsvCustomStatOp CustomStatOp)
{
#if !CSV_PROFILER_MINIMAL
	// LWC_TODO: Double support for FCsvProfiler::RecordCustomStat
	RecordCustomStat(StatName, CategoryIndex, (float)Value, CustomStatOp);
#endif
}

void FCsvProfiler::RecordCustomStat(const FName& StatName, uint32 CategoryIndex, float Value, const ECsvCustomStatOp CustomStatOp)
{
#if !CSV_PROFILER_MINIMAL
	if (GCsvProfilerIsCapturing && GCsvCategoriesEnabled[CategoryIndex])
	{
		UE_AUTORTFM_OPEN
		{
			ValidateFName(StatName);
			FCsvProfilerThreadData::Get().AddCustomStat(StatName, CategoryIndex, Value, CustomStatOp);
		};
	}
#endif
}

void FCsvProfiler::RecordCustomStat(const FName& StatName, uint32 CategoryIndex, double Value, const ECsvCustomStatOp CustomStatOp)
{
#if !CSV_PROFILER_MINIMAL
	// LWC_TODO: Double support for FCsvProfiler::RecordCustomStat
	RecordCustomStat(StatName, CategoryIndex, (float)Value, CustomStatOp);
#endif
}


void FCsvProfiler::RecordCustomStat(const char * StatName, uint32 CategoryIndex, int32 Value, const ECsvCustomStatOp CustomStatOp)
{
#if !CSV_PROFILER_MINIMAL
	if (GCsvProfilerIsCapturing && GCsvCategoriesEnabled[CategoryIndex])
	{
		UE_AUTORTFM_OPEN
		{
			FCsvProfilerThreadData::Get().AddCustomStat(StatName, CategoryIndex, Value, CustomStatOp);
		};
	}
#endif
}

void FCsvProfiler::RecordCustomStat(const FName& StatName, uint32 CategoryIndex, int32 Value, const ECsvCustomStatOp CustomStatOp)
{
#if !CSV_PROFILER_MINIMAL
	if (GCsvProfilerIsCapturing && GCsvCategoriesEnabled[CategoryIndex])
	{
		UE_AUTORTFM_OPEN
		{
			ValidateFName(StatName);
			FCsvProfilerThreadData::Get().AddCustomStat(StatName, CategoryIndex, Value, CustomStatOp);
		};
	}
#endif
}

void FCsvProfiler::RecordEndOfPipeCustomStat(const FName& StatName, uint32 CategoryIndex, double Value, const ECsvCustomStatOp CustomStatOp)
{
#if !CSV_PROFILER_MINIMAL
	if (GCsvProfilerIsCapturing && GCsvCategoriesEnabled[CategoryIndex])
	{
		ValidateFName(StatName);
		FCsvProfilerThreadData::GetEndOfPipe()->AddCustomStat(StatName, CategoryIndex, (float)Value, CustomStatOp);
	}
#endif
}

void FCsvProfiler::PreInit(const TCHAR * Commandline)
{
#if CSV_PROFILER_ALLOW_DEBUG_FEATURES
	// If we're doing boot profiling then we need to init early and make sure the commandline is set
	int32 NumCsvFrames = 0;
	if (FParse::Value(Commandline, TEXT("csvCaptureFrames="), NumCsvFrames))
	{
		if (!FCommandLine::IsInitialized())
		{
			FCommandLine::Set(Commandline);
		}
		// Delay processing thread start
		GDelayProcessingThreadStart = true;
		FCsvProfiler::Get()->InitInternal();
	}
#endif
}

void FCsvProfiler::Init()
{
	InitInternal();
	// It's now safe to start the processing thread
	GDelayProcessingThreadStart = false;
}

void FCsvProfiler::InitInternal()
{
	if (GCsvProfilerInitialized)
	{
		return;
	}
	GCsvProfilerInitialized = true;

#if CSV_PROFILER_ALLOW_DEBUG_FEATURES
	FParse::Value(FCommandLine::Get(), TEXT("csvCaptureOnEventFrameCount="), CaptureOnEventFrameCount);

	GStartOnEvent = new FString();
	FParse::Value(FCommandLine::Get(), TEXT("csvStartOnEvent="), *GStartOnEvent);

	if (GStartOnEvent->IsEmpty())
	{
		delete GStartOnEvent;
		GStartOnEvent = nullptr;
	}

	GStopOnEvent = new FString();
	FParse::Value(FCommandLine::Get(), TEXT("csvStopOnEvent="), *GStopOnEvent);

	if (GStopOnEvent->IsEmpty())
	{
		delete GStopOnEvent;
		GStopOnEvent = nullptr;
	}

	if (FParse::Param(FCommandLine::Get(), TEXT("csvGpuStats")))
	{
		IConsoleVariable* CVarGPUCsvStatsEnabled = IConsoleManager::Get().FindConsoleVariable(TEXT("r.GPUCsvStatsEnabled"));
		if (CVarGPUCsvStatsEnabled)
		{
			CVarGPUCsvStatsEnabled->Set(1);
		}
	}
	if (FParse::Param(FCommandLine::Get(), TEXT("csvTest")))
	{
		GCsvTestingGT = true;
	}
	if (FParse::Param(FCommandLine::Get(), TEXT("csvTestMT")))
	{
		GCsvTestingGT = true;
		GCsvTestingRT = true;
	}
	if (FParse::Param(FCommandLine::Get(), TEXT("csvTestCategoryOnly")))
	{
		GAllCategoriesStartDisabled = true;
		GCsvTestCategoryOnly = true;
	}

	if (FParse::Param(FCommandLine::Get(), TEXT("csvAllCategoriesDisabled")))
	{
		GAllCategoriesStartDisabled = true;
	}

	if (GAllCategoriesStartDisabled)
	{
		FMemory::Memzero(GCsvCategoriesEnabled, sizeof(GCsvCategoriesEnabled));
	}

	FString CsvCategoriesStr;
	if (FParse::Value(FCommandLine::Get(), TEXT("csvCategories="), CsvCategoriesStr, /*bShouldStopOnSeparator*/false))
	{
		TArray<FString> CsvCategories;
		CsvCategoriesStr.ParseIntoArray(CsvCategories, TEXT(","), true);
		for (int i = 0; i < CsvCategories.Num(); i++)
		{
			EnableCategoryByString(CsvCategories[i]);
		}
	}

	FString CsvMetadataStr;
	if (FParse::Value(FCommandLine::Get(), TEXT("csvMetadata="), CsvMetadataStr, false))
	{ 
		TArray<FString> CsvMetadataList;
		CsvMetadataStr.ParseIntoArray(CsvMetadataList, TEXT(","), true);
		for (int i = 0; i < CsvMetadataList.Num(); i++)
		{
			const FString& Metadata = CsvMetadataList[i];
			FString Key;
			FString Value;
			if (Metadata.Split(TEXT("="), &Key, &Value))
			{
				SetMetadata(*Key, *Value);
			}
		}
	}
	if (FParse::Param(FCommandLine::Get(), TEXT("csvNoProcessingThread")))
	{
		GCsvUseProcessingThread = false;
	}
#if CSV_PROFILER_SUPPORT_NAMED_EVENTS
	if (FParse::Param(FCommandLine::Get(), TEXT("csvNamedEvents")))
	{
		GCsvProfilerNamedEventsExclusive = true;
	}
	if (FParse::Param(FCommandLine::Get(), TEXT("csvNamedEventsTiming")))
	{
		GCsvProfilerNamedEventsTiming = true;
	}
#endif
	if (FParse::Param(FCommandLine::Get(), TEXT("csvStatCounts")))
	{
		CVarCsvStatCounts.AsVariable()->Set(1);
	}
	int32 NumCsvFrames = 0;
	if (FParse::Value(FCommandLine::Get(), TEXT("csvCaptureFrames="), NumCsvFrames))
	{
		check(IsInGameThread());
		BeginCapture(NumCsvFrames, FString(), FString(), ECsvProfilerFlags::BootCapture);
		// Call BeginFrame() to start capturing a dummy first "frame"
		BeginFrame();
		// Add a region end marker so we can see where preinit and csv capture started
		EndRegion(CSV_CATEGORY_INDEX_GLOBAL, TEXT("EngineInit/Launch"));
	}
	FParse::Value(FCommandLine::Get(), TEXT("csvRepeat="), GCsvRepeatCount);

	int32 CompressionMode;
	if (FParse::Value(FCommandLine::Get(), TEXT("csvCompression="), CompressionMode))
	{
		switch (CompressionMode)
		{
		case 0: CVarCsvCompressionMode->Set(0); break;
		case 1: CVarCsvCompressionMode->Set(1); break;
		default:
			UE_LOGF(LogCsvProfiler, Warning, "Invalid command line compression mode \"%d\".", CompressionMode);
			break;
		}
	}
	GCsvABTest.InitFromCommandline();

	// Handle -csvExeccmds
	FString CsvExecCommandsStr;
	if (FParse::Value(FCommandLine::Get(), TEXT("-csvExecCmds="), CsvExecCommandsStr, false))
	{
		GCsvFrameExecCmds = new TMap<uint32, TArray<FString>>();
		GCsvFrameModuloExecCmds = new TMap<uint32, TArray<FString>>();

		TArray<FString> CsvExecCommandsList;
		if (CsvExecCommandsStr.ParseIntoArray(CsvExecCommandsList, TEXT(","), true) > 0)
		{
			for (FString FrameAndCommand : CsvExecCommandsList)
			{
				int32 ColonIndex = -1;
				if (FrameAndCommand.FindChar(TEXT(':'), ColonIndex))
				{
					FString FrameStr = FrameAndCommand.Mid(0,ColonIndex).ToLower();
					FString CommandStr = FrameAndCommand.Mid(ColonIndex+1);

					// Modulo frames are prefixed with "r". The command will be executed every n frames
					bool bModuloCommand = FrameStr.StartsWith(TEXT("r"));
					if (bModuloCommand)
					{
						FrameStr = FrameAndCommand.Mid(1, ColonIndex);
					}

					TMap<uint32, TArray<FString>>* ExecCmdList = bModuloCommand ? GCsvFrameModuloExecCmds : GCsvFrameExecCmds;
					uint32 Frame = FCString::Atoi(*FrameStr);
					if (!ExecCmdList->Find(Frame))
					{
						ExecCmdList->Add(Frame, TArray<FString>());
					}
					(*ExecCmdList)[Frame].Add(CommandStr);

					if ( bModuloCommand )
					{
						UE_LOGF(LogCsvProfiler, Display, "Added CsvExecCommand (repeating) - Modulo: %d : %ls", Frame, *CommandStr);
					}
					else
					{
						UE_LOGF(LogCsvProfiler, Display, "Added CsvExecCommand - frame %d : %ls", Frame, *CommandStr);
					}

				}
			}
		}
	}

	if (FParse::Value(FCommandLine::Get(), TEXT("-csvEventExecCmds="), CsvExecCommandsStr, false))
	{
		GCsvEventExecCmds = new TArray<FEventExecCmds>();
		
		TArray<FString> CsvExecCommandsList;
		if (CsvExecCommandsStr.ParseIntoArray(CsvExecCommandsList, TEXT(","), true) > 0)
		{
			for (FString FrameAndCommand : CsvExecCommandsList)
			{
				int32 ColonIndex = FrameAndCommand.Find(TEXT("::"));
				if (ColonIndex != INDEX_NONE)
				{
					FString EventStr = FrameAndCommand.Mid(0,ColonIndex);
					FString CommandStr = FrameAndCommand.Mid(ColonIndex+2);
					if (EventStr.IsEmpty())
					{
						EventStr = TEXT("*");
					}
					bool bAlreadyExists = false;
					for (FEventExecCmds& EventDesc : *GCsvEventExecCmds)
					{
						if (EventDesc.EventWildcard == EventStr)
						{
							bAlreadyExists = true;
							EventDesc.Cmds.Add(CommandStr);
						}
					}
					if (!bAlreadyExists)
					{
						FEventExecCmds& EventDesc = GCsvEventExecCmds->Emplace_GetRef();
						EventDesc.EventWildcard = EventStr;
						EventDesc.Cmds.Add(CommandStr);
					}
					UE_LOGF(LogCsvProfiler, Display, "Added CsvEventExecCommand - event \"%ls\" : %ls", *EventStr, *CommandStr);
				}
			}
		}
	}

#endif // CSV_PROFILER_ALLOW_DEBUG_FEATURES

	// Always disable the CSV profiling thread if the platform does not support threading.
	if (!FPlatformProcess::SupportsMultithreading())
	{
		GCsvUseProcessingThread = false;
	}

	if (GConfig != nullptr && GConfig->IsReadyForUse())
	{
		CsvProfilerReadConfig();
	}
	else
	{
		FCoreDelegates::OnInit.AddStatic(CsvProfilerReadConfig);
	}
}

bool FCsvProfiler::IsCapturing()
{
	check(IsInGameThread());
	return GCsvProfilerIsCapturing;
}

bool FCsvProfiler::IsWritingFile() const
{
	check(IsInGameThread());
	return GCsvProfilerIsWritingFile;
}

bool FCsvProfiler::IsEndCapturePending() const
{
	check(IsInGameThread());
	// Return true if the next command is Stop. If the next command is Start then we ignore, since any further Stop command corresponds to a different capture
	FCsvCaptureCommand CurrentCommand;
	return CommandQueue.Peek(CurrentCommand) && CurrentCommand.CommandType == ECsvCommandType::Stop;
}


bool FCsvProfiler::IsWaitTrackingEnabledOnCurrentThread()
{
	return GCsvTrackWaitsOnAllThreads || GCsvThreadLocalWaitsEnabled;
}

/*Get the current frame capture count*/
int32 FCsvProfiler::GetCaptureFrameNumber() const
{
	return CaptureFrameNumber;
}

int32 FCsvProfiler::GetCaptureFrameNumberRT() const
{
	return CaptureFrameNumberRT;
}


//Get the total frame to capture when we are capturing on event. 
//Example:  -csvStartOnEvent="My Event"
//			-csvCaptureOnEventFrameCount=2500
int32 FCsvProfiler::GetNumFrameToCaptureOnEvent() const
{
	return CaptureOnEventFrameCount;
}

const FGuid& FCsvProfiler::GetCsvID() const
{
	return CsvGUID;
}

/*Get the current number of bytes written to file*/
uint64 FCsvProfiler::GetNumBytesWritten() const
{
	return CsvWriter == nullptr ? 0 : CsvWriter->GetNumBytesWritten();
}

bool FCsvProfiler::EnableCategoryByString(const FString& CategoryName) const
{
	int32 Category = FCsvCategoryData::Get()->GetCategoryIndex(CategoryName);
	if (Category >= 0)
	{
		UE_LOGF(LogCsvProfiler, Log, "Enabled category %ls", *CategoryName);
		GCsvCategoriesEnabled[Category] = true;
		return true;
	}
	UE_LOGF(LogCsvProfiler, Warning, "Error: Can't find category %ls", *CategoryName);
	return false;
}

void FCsvProfiler::EnableCategoryByIndex(uint32 CategoryIndex, bool bEnable) const
{
	check(CategoryIndex < CSV_MAX_CATEGORY_COUNT);
	GCsvCategoriesEnabled[CategoryIndex] = bEnable;
}

bool FCsvProfiler::IsCategoryEnabled(uint32 CategoryIndex) const
{
	check(CategoryIndex < CSV_MAX_CATEGORY_COUNT);
	return GCsvCategoriesEnabled[CategoryIndex];
}

bool FCsvProfiler::IsCapturing_Renderthread()
{
	check(IsInParallelRenderingThread());
	return GCsvProfilerIsCapturingRT;
}

float FCsvProfiler::ProcessStatData(FCsvProcessThreadDataStats* OutStats)
{
	check(IsInCsvProcessingThread());

	QUICK_SCOPE_CYCLE_COUNTER(STAT_FCsvProfiler_ProcessStatData);

	float ElapsedMs = 0.0f;
	if (!IsShuttingDown.GetValue())
	{
		double StartTime = FPlatformTime::Seconds();

		FCsvProcessThreadDataStats Stats;
		if (CsvWriter)
		{
			CsvWriter->Process(Stats);
		}
		ElapsedMs = float(FPlatformTime::Seconds() - StartTime) * 1000.0f;
		CSV_CUSTOM_STAT(CsvProfiler, NumTimestampsProcessed, (int32)Stats.TimestampCount, ECsvCustomStatOp::Accumulate);
		CSV_CUSTOM_STAT(CsvProfiler, NumCustomStatsProcessed, (int32)Stats.CustomStatCount, ECsvCustomStatOp::Accumulate);
		CSV_CUSTOM_STAT(CsvProfiler, NumEventsProcessed, (int32)Stats.EventCount, ECsvCustomStatOp::Accumulate);
		CSV_CUSTOM_STAT(CsvProfiler, ProcessCSVStats, ElapsedMs, ECsvCustomStatOp::Accumulate);

		if (OutStats)
		{
			*OutStats = Stats;
		}		
	}
	return ElapsedMs;
}

TCsvPersistentCustomStat<int32>* FCsvProfiler::GetOrCreatePersistentCustomStatInt(FName Name, int32 CategoryIndex, bool bResetEachFrame)
{
	return GCsvPersistentCustomStats.GetOrCreatePersistentCustomStat<int32>(Name, CategoryIndex, bResetEachFrame);
}

TCsvPersistentCustomStat<float>* FCsvProfiler::GetOrCreatePersistentCustomStatFloat(FName Name, int32 CategoryIndex, bool bResetEachFrame)
{
	return GCsvPersistentCustomStats.GetOrCreatePersistentCustomStat<float>(Name, CategoryIndex, bResetEachFrame);
}

#if CSV_PROFILER_ALLOW_DEBUG_FEATURES

static void SpinWaitMs(double Milliseconds)
{
	double CurrentTime = FPlatformTime::Seconds();
	double SecondsToWait = Milliseconds*0.001;
	double TargetTime = CurrentTime + SecondsToWait;
	while (CurrentTime < TargetTime)
	{
		CurrentTime = FPlatformTime::Seconds();
	}
}

// Simple benchmarking and debugging tests for the csv profiler. Enable with -csvtest, e.g -csvtest -csvcaptureframes=400
void CSVTest()
{
	if (GCsvTestCategoryOnly)
	{
		FMemory::Memzero(GCsvCategoriesEnabled, sizeof(GCsvCategoriesEnabled));
		GCsvCategoriesEnabled[CSV_CATEGORY_INDEX(CsvTest)] = true;
	}
	TCsvPersistentCustomStat<float>* PersistentStatFloat = FCsvProfiler::Get()->GetOrCreatePersistentCustomStatFloat(TEXT("PersistentStatFloat"), CSV_CATEGORY_INDEX(CsvTest));
	PersistentStatFloat->Add(0.15f);
	PersistentStatFloat->Sub(0.1f);

	TCsvPersistentCustomStat<int>* PersistentStatInt = FCsvProfiler::Get()->GetOrCreatePersistentCustomStatInt(TEXT("PersistentStatInt"), CSV_CATEGORY_INDEX(CsvTest));
	PersistentStatInt->Add(15);
	PersistentStatInt->Sub(1);

	uint32 FrameNumber = FCsvProfiler::Get()->GetCaptureFrameNumber();

	static bool bTaskStats = FParse::Param(FCommandLine::Get(), TEXT("csvTestTasks"));
	if (bTaskStats)
	{
		ParallelFor(4, [FrameNumber](int32 Index)
			{
				CSV_SCOPED_TIMING_STAT(CsvTest, TaskTimer);
				if (!IsInGameThread())
				{
					CSV_CUSTOM_STAT(CsvTest, TaskCustomStatSet, 0.5, ECsvCustomStatOp::Set);
					CSV_CUSTOM_STAT(CsvTest, TaskCustomStatSet, 1.0, ECsvCustomStatOp::Set);
					CSV_CUSTOM_STAT(CsvTest, TaskCustomStatAccumulate, 0.5, ECsvCustomStatOp::Accumulate);
					CSV_CUSTOM_STAT(CsvTest, TaskCustomStatAccumulate, 0.5, ECsvCustomStatOp::Accumulate);
					CSV_CUSTOM_STAT(CsvTest, TaskCustomStatMax, 0.5, ECsvCustomStatOp::Max);
					CSV_CUSTOM_STAT(CsvTest, TaskCustomStatMax, 1.0, ECsvCustomStatOp::Max);

					if (FrameNumber % 321 == 0)
					{
						CSV_CUSTOM_STAT(CsvTest, TaskSparse, 1.0, ECsvCustomStatOp::Set);
					}
				}
				SpinWaitMs(1.0);
			},
			EParallelForFlags::BackgroundPriority);
	}

	CSV_SCOPED_TIMING_STAT(CsvTest, CsvTestStat);
	CSV_CUSTOM_STAT(CsvTest, CaptureFrameNumber, int32(FrameNumber), ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT(CsvTest, SameCustomStat, 1, ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT(CsvTest, SameCustomStat, 1, ECsvCustomStatOp::Accumulate);
	for (int i = 0; i < 3; i++)
	{
		CSV_SCOPED_TIMING_STAT(CsvTest, RepeatStat1MS);
		SpinWaitMs(1.0);
	}

	{
		// This stat measures the overhead of submitting 10k timing stat scopes in a frame. 
		// Multiply the ms result by 100 to get the per-scope cost in ns
		// (currently ~150ns/scope on last-gen consoles if CSVPROFILERTRACE_ENABLED is 0)
		// Note that each scope emits two timestamps, so the per-timestamp cost is half this
		CSV_SCOPED_TIMING_STAT(CsvTest, TimerStatTimer);
		for (int i = 0; i < 2500; i++)
		{
			CSV_SCOPED_TIMING_STAT(CsvTest, BeginEndbenchmarkInner0);
			CSV_SCOPED_TIMING_STAT(CsvTest, BeginEndbenchmarkInner1);
			CSV_SCOPED_TIMING_STAT(CsvTest, BeginEndbenchmarkInner2);
			CSV_SCOPED_TIMING_STAT(CsvTest, BeginEndbenchmarkInner3);
		}
	}

	{
		CSV_SCOPED_TIMING_STAT(CsvTest, CustomStatTimer);
		for (int i = 0; i < 100; i++)
		{
			CSV_CUSTOM_STAT(CsvTest, SetStat_99, i, ECsvCustomStatOp::Set); // Should be 99
			CSV_CUSTOM_STAT(CsvTest, MaxStat_99, 99 - i, ECsvCustomStatOp::Max); // Should be 99
			CSV_CUSTOM_STAT(CsvTest, MinStat_0, i, ECsvCustomStatOp::Min); // Should be 0
			CSV_CUSTOM_STAT(CsvTest, AccStat_4950, i, ECsvCustomStatOp::Accumulate); // Should be 4950
		}
		if (FrameNumber > 100)
		{
			CSV_SCOPED_TIMING_STAT(CsvTest, TimerOver100);
			CSV_CUSTOM_STAT(CsvTest, CustomStatOver100, int32(FrameNumber - 100), ECsvCustomStatOp::Set);
		}
	}
	{
		CSV_SCOPED_TIMING_STAT(CsvTest, EventTimer);
		if (FrameNumber % 20 < 2)
		{
			CSV_EVENT(CsvTest, TEXT("This is frame %d"), GFrameNumber);
		}
		if (FrameNumber % 50 == 0)
		{
			for (int i = 0; i < 5; i++)
			{
				CSV_EVENT(CsvTest, TEXT("Multiple Event %d"), i);
			}
		}
	}
	{
		CSV_SCOPED_TIMING_STAT_EXCLUSIVE(ExclusiveLevel0);
		{
			CSV_SCOPED_TIMING_STAT_EXCLUSIVE(ExclusiveLevel1);
			CSV_SCOPED_TIMING_STAT(CsvTest, NonExclusiveTestLevel1);
			FPlatformProcess::Sleep(0.002f);
			{
				CSV_SCOPED_TIMING_STAT_EXCLUSIVE(ExclusiveLevel2);
				CSV_SCOPED_TIMING_STAT(CsvTest, NonExclusiveTestLevel2);
				FPlatformProcess::Sleep(0.003f);
			}
		}
		FPlatformProcess::Sleep(0.001f);
	}
	{
		CSV_SCOPED_TIMING_STAT(CsvTest, ExclusiveTimerStatTimer);
		for (int i = 0; i < 100; i++)
		{
			CSV_SCOPED_TIMING_STAT_EXCLUSIVE(ExclusiveBeginEndbenchmarkInner0);
			CSV_SCOPED_TIMING_STAT_EXCLUSIVE(ExclusiveBeginEndbenchmarkInner1);
			CSV_SCOPED_TIMING_STAT_EXCLUSIVE(ExclusiveBeginEndbenchmarkInner2);
			CSV_SCOPED_TIMING_STAT_EXCLUSIVE(ExclusiveBeginEndbenchmarkInner3);
		}
	}
}

#endif // CSV_PROFILER_ALLOW_DEBUG_FEATURES

#endif // CSV_PROFILER
//UE_ENABLE_OPTIMIZATION