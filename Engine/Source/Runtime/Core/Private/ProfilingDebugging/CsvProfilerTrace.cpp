// Copyright Epic Games, Inc. All Rights Reserved.
#include "ProfilingDebugging/CsvProfilerTrace.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "ProfilingDebugging/CountersTrace.h"
#include "HAL/PlatformTime.h"
#include "UObject/NameTypes.h"
#include "Trace/Trace.inl"

#if CSVPROFILERTRACE_ENABLED

UE_TRACE_EVENT_BEGIN(CsvProfiler, RegisterCategory, NoSync|Important)
	UE_TRACE_EVENT_FIELD(int32, Index)
	UE_TRACE_EVENT_FIELD(UE::Trace::AnsiString, Name)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(CsvProfiler, DefineInlineStat, NoSync|Important)
	UE_TRACE_EVENT_FIELD(uint64, StatId)
	UE_TRACE_EVENT_FIELD(int32, CategoryIndex)
	UE_TRACE_EVENT_FIELD(UE::Trace::AnsiString, Name)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(CsvProfiler, DefineDeclaredStat, NoSync|Important)
	UE_TRACE_EVENT_FIELD(uint64, StatId)
	UE_TRACE_EVENT_FIELD(int32, CategoryIndex)
	UE_TRACE_EVENT_FIELD(UE::Trace::AnsiString, Name)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(CsvProfiler, BeginStat)
	UE_TRACE_EVENT_FIELD(uint64, StatId)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(CsvProfiler, EndStat)
	UE_TRACE_EVENT_FIELD(uint64, StatId)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(CsvProfiler, BeginExclusiveStat)
	UE_TRACE_EVENT_FIELD(uint64, StatId)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(CsvProfiler, EndExclusiveStat)
	UE_TRACE_EVENT_FIELD(uint64, StatId)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(CsvProfiler, CustomStatInt)
	UE_TRACE_EVENT_FIELD(uint64, StatId)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(int32, Value)
	UE_TRACE_EVENT_FIELD(uint8, OpType)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(CsvProfiler, CustomStatFloat)
	UE_TRACE_EVENT_FIELD(uint64, StatId)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(float, Value)
	UE_TRACE_EVENT_FIELD(uint8, OpType)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(CsvProfiler, Event)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(int32, CategoryIndex)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, Text)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(CsvProfiler, BeginCapture)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(uint32, RenderThreadId)
	UE_TRACE_EVENT_FIELD(uint32, RHIThreadId)
	UE_TRACE_EVENT_FIELD(bool, EnableCounts)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, FileName)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(CsvProfiler, EndCapture)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(CsvProfiler, Metadata)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, Key)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, Value)
UE_TRACE_EVENT_END()

struct FCsvProfilerTraceInternal
{
	union FStatId
	{
		struct
		{
			uint64 IsFName : 1;
			uint64 CategoryIndex : 11;
			uint64 FNameOrCString : 52;
		} Fields;
		uint64 Hash;
	};

	FORCEINLINE static uint64 GetStatId(const char* StatName, int32 CategoryIndex)
	{
		FStatId StatId;
		StatId.Fields.IsFName = false;
		StatId.Fields.CategoryIndex = CategoryIndex;
		StatId.Fields.FNameOrCString = uint64(StatName);
		return StatId.Hash;
	}

	FORCEINLINE static uint64 GetStatId(const FName& StatName, int32 CategoryIndex)
	{
		FStatId StatId;
		StatId.Fields.IsFName = true;
		StatId.Fields.CategoryIndex = CategoryIndex;
		StatId.Fields.FNameOrCString = StatName.GetComparisonIndex().ToUnstableInt();
		return StatId.Hash;
	}
};

void FCsvProfilerTrace::OutputRegisterCategory(int32 Index, const TCHAR* Name)
{
	uint16 NameLen = uint16(FCString::Strlen(Name));
	UE_TRACE_LOG(CsvProfiler, RegisterCategory, CountersChannel, NameLen * sizeof(ANSICHAR))
		<< RegisterCategory.Index(Index)
		<< RegisterCategory.Name(Name, NameLen);
}

void FCsvProfilerTrace::OutputInlineStat(const char* StatName, int32 CategoryIndex)
{
	uint16 NameLen = uint16(strlen(StatName));
	UE_TRACE_LOG(CsvProfiler, DefineInlineStat, CountersChannel, NameLen * sizeof(ANSICHAR))
		<< DefineInlineStat.StatId(FCsvProfilerTraceInternal::GetStatId(StatName, CategoryIndex))
		<< DefineInlineStat.CategoryIndex(CategoryIndex)
		<< DefineInlineStat.Name(StatName, NameLen);
}

CSV_DECLARE_CATEGORY_EXTERN(Exclusive);

void FCsvProfilerTrace::OutputInlineStatExclusive(const char* StatName)
{
	OutputInlineStat(StatName, CSV_CATEGORY_INDEX(Exclusive));
}

void FCsvProfilerTrace::OutputDeclaredStat(const FName& StatName, int32 CategoryIndex)
{
	TCHAR NameString[NAME_SIZE];
	StatName.GetPlainNameString(NameString);
	uint16 NameLen = uint16(FCString::Strlen(NameString));
	UE_TRACE_LOG(CsvProfiler, DefineDeclaredStat, CountersChannel, NameLen * sizeof(ANSICHAR))
		<< DefineDeclaredStat.StatId(FCsvProfilerTraceInternal::GetStatId(StatName, CategoryIndex))
		<< DefineDeclaredStat.CategoryIndex(CategoryIndex)
		<< DefineDeclaredStat.Name(NameString, NameLen);
}

void FCsvProfilerTrace::OutputBeginStat(const char* StatName, int32 CategoryIndex, uint64 Cycles)
{
	UE_TRACE_LOG(CsvProfiler, BeginStat, CountersChannel)
		<< BeginStat.StatId(FCsvProfilerTraceInternal::GetStatId(StatName, CategoryIndex))
		<< BeginStat.Cycle(Cycles);
}

void FCsvProfilerTrace::OutputBeginStat(const FName& StatName, int32 CategoryIndex, uint64 Cycles)
{
	UE_TRACE_LOG(CsvProfiler, BeginStat, CountersChannel)
		<< BeginStat.StatId(FCsvProfilerTraceInternal::GetStatId(StatName, CategoryIndex))
		<< BeginStat.Cycle(Cycles);
}

void FCsvProfilerTrace::OutputEndStat(const char* StatName, int32 CategoryIndex, uint64 Cycles)
{
	UE_TRACE_LOG(CsvProfiler, EndStat, CountersChannel)
		<< EndStat.StatId(FCsvProfilerTraceInternal::GetStatId(StatName, CategoryIndex))
		<< EndStat.Cycle(Cycles);
}

void FCsvProfilerTrace::OutputEndStat(const FName& StatName, int32 CategoryIndex, uint64 Cycles)
{
	UE_TRACE_LOG(CsvProfiler, EndStat, CountersChannel)
		<< EndStat.StatId(FCsvProfilerTraceInternal::GetStatId(StatName, CategoryIndex))
		<< EndStat.Cycle(Cycles);
}

void FCsvProfilerTrace::OutputBeginExclusiveStat(const char* StatName, int32 CategoryIndex, uint64 Cycles)
{
	UE_TRACE_LOG(CsvProfiler, BeginExclusiveStat, CountersChannel)
		<< BeginExclusiveStat.StatId(FCsvProfilerTraceInternal::GetStatId(StatName, CategoryIndex))
		<< BeginExclusiveStat.Cycle(Cycles);
}

void FCsvProfilerTrace::OutputEndExclusiveStat(const char* StatName, int32 CategoryIndex, uint64 Cycles)
{
	UE_TRACE_LOG(CsvProfiler, EndExclusiveStat, CountersChannel)
		<< EndExclusiveStat.StatId(FCsvProfilerTraceInternal::GetStatId(StatName, CategoryIndex))
		<< EndExclusiveStat.Cycle(Cycles);
}

void FCsvProfilerTrace::OutputCustomStat(const char* StatName, int32 CategoryIndex, int32 Value, uint8 OpType, uint64 Cycles)
{
	UE_TRACE_LOG(CsvProfiler, CustomStatInt, CountersChannel)
		<< CustomStatInt.Cycle(Cycles)
		<< CustomStatInt.StatId(FCsvProfilerTraceInternal::GetStatId(StatName, CategoryIndex))
		<< CustomStatInt.Value(Value)
		<< CustomStatInt.OpType(OpType);
}

void FCsvProfilerTrace::OutputCustomStat(const FName& StatName, int32 CategoryIndex, int32 Value, uint8 OpType, uint64 Cycles)
{
	UE_TRACE_LOG(CsvProfiler, CustomStatInt, CountersChannel)
		<< CustomStatInt.Cycle(Cycles)
		<< CustomStatInt.StatId(FCsvProfilerTraceInternal::GetStatId(StatName, CategoryIndex))
		<< CustomStatInt.Value(Value)
		<< CustomStatInt.OpType(OpType);
}

void FCsvProfilerTrace::OutputCustomStat(const char* StatName, int32 CategoryIndex, float Value, uint8 OpType, uint64 Cycles)
{
	UE_TRACE_LOG(CsvProfiler, CustomStatFloat, CountersChannel)
		<< CustomStatFloat.Cycle(Cycles)
		<< CustomStatFloat.StatId(FCsvProfilerTraceInternal::GetStatId(StatName, CategoryIndex))
		<< CustomStatFloat.Value(Value)
		<< CustomStatFloat.OpType(OpType);
}

void FCsvProfilerTrace::OutputCustomStat(const FName& StatName, int32 CategoryIndex, float Value, uint8 OpType, uint64 Cycles)
{
	UE_TRACE_LOG(CsvProfiler, CustomStatFloat, CountersChannel)
		<< CustomStatFloat.Cycle(Cycles)
		<< CustomStatFloat.StatId(FCsvProfilerTraceInternal::GetStatId(StatName, CategoryIndex))
		<< CustomStatFloat.Value(Value)
		<< CustomStatFloat.OpType(OpType);
}

void FCsvProfilerTrace::OutputBeginCapture(const TCHAR* Filename, uint32 RenderThreadId, uint32 RHIThreadId, const char* DefaultWaitStatName, bool bEnableCounts)
{
	OutputInlineStat(DefaultWaitStatName, CSV_CATEGORY_INDEX(Exclusive));
	UE_TRACE_LOG(CsvProfiler, BeginCapture, CountersChannel)
		<< BeginCapture.Cycle(FPlatformTime::Cycles64())
		<< BeginCapture.RenderThreadId(RenderThreadId)
		<< BeginCapture.RHIThreadId(RHIThreadId)
		<< BeginCapture.EnableCounts(bEnableCounts)
		<< BeginCapture.FileName(Filename);
}

void FCsvProfilerTrace::OutputEvent(const TCHAR* Text, int32 CategoryIndex, uint64 Cycles)
{
	UE_TRACE_LOG(CsvProfiler, Event, CountersChannel)
		<< Event.Cycle(Cycles)
		<< Event.CategoryIndex(CategoryIndex)
		<< Event.Text(Text);
}

void FCsvProfilerTrace::OutputEndCapture()
{
	UE_TRACE_LOG(CsvProfiler, EndCapture, CountersChannel)
		<< EndCapture.Cycle(FPlatformTime::Cycles64());
}

void FCsvProfilerTrace::OutputMetadata(const TCHAR* Key, const TCHAR* Value)
{
	UE_TRACE_LOG(CsvProfiler, Metadata, CountersChannel)
		<< Metadata.Key(Key)
		<< Metadata.Value(Value);
}

#endif
