// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProfilingDebugging/ExternalProfiler.h"
#include "Algo/Find.h"
#include "Logging/LogMacros.h"
#include "Misc/Parse.h"
#include "Misc/CommandLine.h"
#include "Features/IModularFeatures.h"
#include "Misc/ConfigCacheIni.h"

#if UE_EXTERNAL_PROFILING_ENABLED

DEFINE_LOG_CATEGORY_STATIC( LogExternalProfiler, Log, All );

FExternalProfiler::FExternalProfiler()
{
}

FName FExternalProfiler::GetFeatureName()
{
	static FName ProfilerFeatureName( "ExternalProfiler" );
	return ProfilerFeatureName;
}

bool FActiveExternalProfilerBase::bDidInitialize = false;

FExternalProfiler* FActiveExternalProfilerBase::ActiveProfiler = nullptr;

std::atomic<bool> FActiveExternalProfilerBase::bIsRecording = false;

std::atomic<bool> FActiveExternalProfilerBase::bEnableScopedEvents = true;

static FExternalProfiler* FindProfilerByName(TArrayView<FExternalProfiler*> Profilers, FStringView Name)
{
	FExternalProfiler** Found = Algo::FindByPredicate(
		Profilers,
		[&Name](const FExternalProfiler* Profiler)
		{
			return Name == Profiler->GetProfilerName();
		});
	return Found ? *Found : nullptr;
}

FExternalProfiler* FActiveExternalProfilerBase::InitActiveProfiler()
{
	// Create profiler on demand.
	if (ActiveProfiler == nullptr && !bDidInitialize && FCommandLine::IsInitialized())
	{
		const FName FeatureName = FExternalProfiler::GetFeatureName();
		TArray<FExternalProfiler*> AvailableProfilers = IModularFeatures::Get().GetModularFeatureImplementations<FExternalProfiler>(FeatureName);

		for (FExternalProfiler* CurProfiler : AvailableProfilers)
		{
			check(CurProfiler != nullptr);

#if 0
			// Logging disabled here as it can cause a stack overflow whilst flushing logs during EnginePreInit
			UE_LOGF(LogExternalProfiler, Log, "Found external profiler: %ls", CurProfiler->GetProfilerName());
#endif

			// Check to see if the profiler was specified on the command-line (e.g., "-VTune")
			if (FParse::Param(FCommandLine::Get(), CurProfiler->GetProfilerName()))
			{
				ActiveProfiler = CurProfiler;
			}
		}

#if 0
		// Logging disabled here as it can cause a stack overflow whilst flushing logs during EnginePreInit
		if (ActiveProfiler != nullptr)
		{
			UE_LOGF(LogExternalProfiler, Log, "Using external profiler: %ls", ActiveProfiler->GetProfilerName());
		}
		else
		{
			UE_LOGF(LogExternalProfiler, Log, "No external profilers were discovered.  External profiling features will not be available.");
		}
#endif

		if (ActiveProfiler == nullptr)
		{
			FString ProfilerName = FPlatformMisc::GetEnvironmentVariable(TEXT("UE_EXTERNAL_PROFILER"));
			if (!ProfilerName.IsEmpty())
			{
				ActiveProfiler = FindProfilerByName(AvailableProfilers, ProfilerName);
			}
		}

		if (ActiveProfiler == nullptr)
		{
			FString ProfilerName = GConfig->GetStr(TEXT("Core.ProfilingDebugging"), TEXT("ExternalProfiler"), GEngineIni);
			if (!ProfilerName.IsEmpty())
			{
				ActiveProfiler = FindProfilerByName(AvailableProfilers, ProfilerName);
			}
		}

		// Don't try to initialize again this session
		bDidInitialize = true;
	}

	return ActiveProfiler;
}

bool FActiveExternalProfilerBase::SetActiveProfilerRecording(bool bRecording)
{
	bool bWasRecording = bIsRecording.exchange(bRecording);

	if (FExternalProfiler* Profiler = GetActiveProfiler())
	{
		if (bWasRecording && !bRecording)
		{
			Profiler->ProfilerPauseFunction();
		}
		else if (!bWasRecording && bRecording)
		{
			Profiler->ProfilerResumeFunction();
		}
	}

	return bWasRecording;
}

bool FActiveExternalProfilerBase::IsActiveProfilerRecording()
{
	return bIsRecording.load(std::memory_order_relaxed);
}

void FActiveExternalProfilerBase::EnableScopedEvents(bool bEnable)
{
	bool bPreviousEnable = bEnableScopedEvents.exchange(bEnable, std::memory_order_relaxed);

	if (bPreviousEnable == bEnable)
	{
		return;
	}

	FExternalProfiler* Profiler = GetActiveProfiler();
	if (!Profiler)
	{
		return;
	}
	
	Profiler->OnEnableScopedEventsChanged(bEnable);
}

bool FActiveExternalProfilerBase::AreScopedEventsEnabled()
{
	return bEnableScopedEvents.load(std::memory_order_relaxed);
}

void FScopedExternalProfilerBase::StartScopedTimer( const bool bWantPause )
{
	bWasRecording = FActiveExternalProfilerBase::SetActiveProfilerRecording(!bWantPause);
}

void FScopedExternalProfilerBase::StopScopedTimer()
{
	FActiveExternalProfilerBase::SetActiveProfilerRecording(bWasRecording);
}

#endif	// UE_EXTERNAL_PROFILING_ENABLED
