// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProfilingDebugging/Histogram.h"

#include "Logging/LogCategory.h"
#include "Templates/Function.h"
#include "Trace/Detail/Channel.h"

DEFINE_LOG_CATEGORY(LogHistograms);

//////////////////////////////////////////////////////////////////////
// FHistogram

void FHistogram::InitLinear(double MinTime, double MaxTime, double BinSize)
{
	SumOfAllMeasures = 0.0;
	CountOfAllMeasures = 0;
	MinimalMeasurement = DBL_MAX;
	MaximalMeasurement = -DBL_MAX;

	Bins.Empty();

	double CurrentBinMin = MinTime;
	while (CurrentBinMin < MaxTime)
	{
		Bins.Add(FBin(CurrentBinMin, CurrentBinMin + BinSize));
		CurrentBinMin += BinSize;
	}
	Bins.Add(FBin(CurrentBinMin));	// catch-all-above bin
}

void FHistogram::InitHitchTracking()
{
	SumOfAllMeasures = 0.0;
	CountOfAllMeasures = 0;
	MinimalMeasurement = DBL_MAX;
	MaximalMeasurement = -DBL_MAX;

	Bins.Empty();

	Bins.Add(FBin(0.0, 9.0));		// >= 120 fps
	Bins.Add(FBin(9.0, 17.0));		// 60 - 120 fps
	Bins.Add(FBin(17.0, 34.0));		// 30 - 60 fps
	Bins.Add(FBin(34.0, 50.0));		// 20 - 30 fps
	Bins.Add(FBin(50.0, 67.0));		// 15 - 20 fps
	Bins.Add(FBin(67.0, 100.0));	// 10 - 15 fps
	Bins.Add(FBin(100.0, 200.0));	// 5 - 10 fps
	Bins.Add(FBin(200.0, 300.0));	// < 5 fps
	Bins.Add(FBin(300.0, 500.0));
	Bins.Add(FBin(500.0, 750.0));
	Bins.Add(FBin(750.0, 1000.0));
	Bins.Add(FBin(1000.0, 1500.0));
	Bins.Add(FBin(1500.0, 2000.0));
	Bins.Add(FBin(2000.0, 2500.0));
	Bins.Add(FBin(2500.0, 5000.0));
	Bins.Add(FBin(5000.0));
}

void FHistogram::InitFromArray(TArrayView<const double> Thresholds)
{
	SumOfAllMeasures = 0.0;
	CountOfAllMeasures = 0;
	MinimalMeasurement = DBL_MAX;
	MaximalMeasurement = -DBL_MAX;
	Bins.Empty();

	for (int32 Index = 0; Index < Thresholds.Num(); ++Index)
	{
		const int32 NextIndex = Index + 1;
		if (NextIndex == Thresholds.Num())
		{
			Bins.Emplace(Thresholds[Index]);
		}
		else
		{
			Bins.Emplace(Thresholds[Index], Thresholds[NextIndex]);
		}
	}
}

void FHistogram::Reset()
{
	for (FBin& Bin : Bins)
	{
		Bin.Count = 0;
		Bin.Sum = 0.0;
	}
	SumOfAllMeasures = 0.0;
	CountOfAllMeasures = 0;
	MinimalMeasurement = DBL_MAX;
	MaximalMeasurement = -DBL_MAX;
}

void FHistogram::AddMeasurement(double ValueForBinning, double MeasurementValue)
{
	if (LIKELY(Bins.Num()))
	{
		FBin& FirstBin = Bins[0];
		if (UNLIKELY(ValueForBinning < FirstBin.MinValue))
		{
			return;
		}

		for (int BinIdx = 0, LastBinIdx = Bins.Num() - 1; BinIdx < LastBinIdx; ++BinIdx)
		{
			FBin& Bin = Bins[BinIdx];
			if (Bin.UpperBound > ValueForBinning)
			{
				++Bin.Count;
				++CountOfAllMeasures;
				Bin.Sum += MeasurementValue;
				SumOfAllMeasures += MeasurementValue;
				MinimalMeasurement = FMath::Min(MinimalMeasurement, MeasurementValue);
				MaximalMeasurement = FMath::Max(MaximalMeasurement, MeasurementValue);
				return;
			}
		}

		// if we got here, ValueForBinning did not fit in any of the previous bins
		FBin& LastBin = Bins.Last();
		++LastBin.Count;
		++CountOfAllMeasures;
		LastBin.Sum += MeasurementValue;
		SumOfAllMeasures += MeasurementValue;
		MinimalMeasurement = FMath::Min(MinimalMeasurement, MeasurementValue);
		MaximalMeasurement = FMath::Max(MaximalMeasurement, MeasurementValue);
	}
}

FString FHistogram::DumpToJsonString(TFunctionRef<FString (double, double)> ConvertBinToLabel) const
{
	// {"Bin":"Name","Count":Count,"Sum":Sum}
	FString Result;
	if (LIKELY(Bins.Num()))
	{
		Result += TEXT('[');
		for (int BinIdx = 0, LastBinIdx = Bins.Num() - 1; BinIdx < LastBinIdx+1; ++BinIdx)
		{
			const FBin& Bin = Bins[BinIdx];
			if (BinIdx != 0)
			{
				Result += TEXT(',');
			}
			Result += FString::Printf(TEXT("{\"Bin\":\"%s\",\"Count\":%d,\"Sum\":%.5f}"), *ConvertBinToLabel(Bin.MinValue, Bin.UpperBound), Bin.Count, Bin.Sum);
		}
		Result.AppendChar(TEXT(']'));
	}
	return Result;
}

FString FHistogram::DumpToJsonString() const
{
	return DumpToJsonString(&DefaultConvertBinToLabel);
}

FString FHistogram::DumpToJsonString2(TFunctionRef<FString (double, double)> ConvertBinToLabel) const
{
	// {"BinName":{"Count":Count,"Sum":Sum}}
	FString Result;
	if (LIKELY(Bins.Num()))
	{
		Result += TEXT('[');
		for (int BinIdx = 0, LastBinIdx = Bins.Num() - 1; BinIdx < LastBinIdx+1; ++BinIdx)
		{
			const FBin& Bin = Bins[BinIdx];
			if (BinIdx != 0)
			{
				Result += TEXT(',');
			}
			Result += FString::Printf(TEXT("{\"%s\":{\"Count\":%d,\"Sum\":%.5f}}"), *ConvertBinToLabel(Bin.MinValue, Bin.UpperBound), Bin.Count, Bin.Sum);
		}
		Result.AppendChar(TEXT(']'));
	}
	return Result;
}

FString FHistogram::DumpToJsonString2() const
{
	return DumpToJsonString2(&DefaultConvertBinToLabel);
}

FString FHistogram::DefaultConvertBinToLabel(double MinValue, double UpperBound)
{
	if (UpperBound >= FLT_MAX)
	{
		return FString::Printf(TEXT("%d_Plus"), (int)MinValue);
	}
	else
	{
		return FString::Printf(TEXT("%d_%d"), (int)MinValue, (int)UpperBound);
	}
}

void FHistogram::DumpToLog(const FString& HistogramName)
{
	UE_LOGF(LogHistograms, Log, "Histogram '%ls': %d bins", *HistogramName, Bins.Num());

	if (LIKELY(Bins.Num()))
	{
		double TotalSum = 0;
		double TotalObservations = 0;

		for (int BinIdx = 0, LastBinIdx = Bins.Num() - 1; BinIdx < LastBinIdx; ++BinIdx)
		{
			FBin& Bin = Bins[BinIdx];
			UE_LOGF(LogHistograms, Log, "Bin %4.0f - %4.0f: %5d observation(s) which sum up to %f", Bin.MinValue, Bin.UpperBound, Bin.Count, Bin.Sum);

			TotalObservations += Bin.Count;
			TotalSum += Bin.Sum;
		}

		FBin& LastBin = Bins.Last();
		UE_LOGF(LogHistograms, Log, "Bin %4.0f +     : %5d observation(s) which sum up to %f", LastBin.MinValue, LastBin.Count, LastBin.Sum);
		TotalObservations += LastBin.Count;
		TotalSum += LastBin.Sum;

		if (TotalObservations > 0)
		{
			UE_LOGF(LogHistograms, Log, "Average value for observation: %f", TotalSum / TotalObservations);
		}
	}
}
