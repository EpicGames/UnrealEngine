// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProfilingDebugging/HttpTrace.h"

#if UE_TRACE_HTTP_ENABLED

#include "HAL/PlatformTime.h"
#include "IO/IoChunkId.h"
#include "Trace/Trace.h"

////////////////////////////////////////////////////////////////////////////////////////////////////
UE_TRACE_CHANNEL_DEFINE(HttpChannel, "HTTP dispatcher and request lifecycle events: dispatcher/category creation, request start (URL, priority), chunk range additions, and request completion (status code, content length).")

////////////////////////////////////////////////////////////////////////////////////////////////////
UE_TRACE_EVENT_BEGIN(Http, DispatcherCreated, NoSync)
	UE_TRACE_EVENT_FIELD(uint64, Dispatcher)
	UE_TRACE_EVENT_FIELD(UE::Trace::AnsiString, Name)
UE_TRACE_EVENT_END()

void FHttpTrace::DispatcherCreated(uint64 Dispatcher, const ANSICHAR* Name)
{
	UE_TRACE_LOG(Http, DispatcherCreated, HttpChannel)
		<< DispatcherCreated.Dispatcher(Dispatcher)
		<< DispatcherCreated.Name(Name);
}

////////////////////////////////////////////////////////////////////////////////////////////////////
UE_TRACE_EVENT_BEGIN(Http, CategoryCreated, NoSync)
	UE_TRACE_EVENT_FIELD(uint8, Category)
	UE_TRACE_EVENT_FIELD(UE::Trace::AnsiString, Name)
UE_TRACE_EVENT_END()

void FHttpTrace::CategoryCreated(uint8 Category, const ANSICHAR* Name)
{
	UE_TRACE_LOG(Http, CategoryCreated, HttpChannel)
		<< CategoryCreated.Category(Category)
		<< CategoryCreated.Name(Name);
}

////////////////////////////////////////////////////////////////////////////////////////////////////
UE_TRACE_EVENT_BEGIN(Http, RequestStarted, NoSync)
	UE_TRACE_EVENT_FIELD(uint64, Dispatcher)
	UE_TRACE_EVENT_FIELD(uint64, Request)
	UE_TRACE_EVENT_FIELD(uint64, StartTime)
	UE_TRACE_EVENT_FIELD(int32, Priority)
	UE_TRACE_EVENT_FIELD(uint32, Category)
	UE_TRACE_EVENT_FIELD(UE::Trace::AnsiString, Url)
UE_TRACE_EVENT_END()

void FHttpTrace::RequestStarted(uint64 Dispatcher, uint64 Request, const ANSICHAR* Url, int32 Priority, uint8 Category)
{
	UE_TRACE_LOG(Http, RequestStarted, HttpChannel)
		<< RequestStarted.Dispatcher(Dispatcher)
		<< RequestStarted.Request(Request)
		<< RequestStarted.StartTime(FPlatformTime::Cycles64())
		<< RequestStarted.Priority(Priority)
		<< RequestStarted.Category(Category)
		<< RequestStarted.Url(Url);
}

////////////////////////////////////////////////////////////////////////////////////////////////////
UE_TRACE_EVENT_BEGIN(Http, ChunkRangeAdded, NoSync)
	UE_TRACE_EVENT_FIELD(uint64, Request)
	UE_TRACE_EVENT_FIELD(uint32, Start)
	UE_TRACE_EVENT_FIELD(uint32, End)
	UE_TRACE_EVENT_FIELD(uint8[], ChunkId)
UE_TRACE_EVENT_END()

void FHttpTrace::ChunkRangeAdded(uint64 Request, const FIoChunkId& ChunkId, uint32 Start, uint32 End)
{
	UE_TRACE_LOG(Http, ChunkRangeAdded, HttpChannel)
		<< ChunkRangeAdded.Request(Request)
		<< ChunkRangeAdded.Start(Start)
		<< ChunkRangeAdded.End(End)
		<< ChunkRangeAdded.ChunkId(reinterpret_cast<const uint8*>(&ChunkId), uint32(sizeof(FIoChunkId)));
}

////////////////////////////////////////////////////////////////////////////////////////////////////
UE_TRACE_EVENT_BEGIN(Http, RequestCompleted, NoSync)
	UE_TRACE_EVENT_FIELD(uint64, Request)
	UE_TRACE_EVENT_FIELD(uint64, CompletionTime)
	UE_TRACE_EVENT_FIELD(uint32, StatusCode)
	UE_TRACE_EVENT_FIELD(uint32, ContentLength)
	UE_TRACE_EVENT_FIELD(UE::Trace::AnsiString, Host)
UE_TRACE_EVENT_END()

void FHttpTrace::RequestCompleted(uint64 Request, const ANSICHAR* Host, uint32 StatusCode, uint32 ContentLength)
{
	UE_TRACE_LOG(Http, RequestCompleted, HttpChannel)
		<< RequestCompleted.Request(Request)
		<< RequestCompleted.CompletionTime(FPlatformTime::Cycles64())
		<< RequestCompleted.StatusCode(StatusCode)
		<< RequestCompleted.ContentLength(ContentLength)
		<< RequestCompleted.Host(Host);
}

#endif //UE_TRACE_HTTP_ENABLED
