// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProfilingDebugging/InstanceCounter.h"
#include "Misc/ScopeLock.h"
#include "Logging/LogMacros.h"
#include "HAL/IConsoleManager.h"

DEFINE_LOG_CATEGORY_STATIC(LogInstanceCount, Log, All);

FInstanceCountingObject::FGlobalVars* FInstanceCountingObject::Globals = nullptr;

/**
 *	used to dump instance counts at the console or with associated commands (e.g. memreport)
 */
static FAutoConsoleCommandWithOutputDevice FInstanceCountingDumpCommand(
	TEXT("LogCountedInstances"),
	TEXT("Dumps count of all tracked FInstanceCountingObject's"),
	FConsoleCommandWithOutputDeviceDelegate::CreateStatic(&FInstanceCountingObject::LogCounts)
);

FInstanceCountingObject::FGlobalVars& FInstanceCountingObject::GetGlobals()
{
	if (Globals == nullptr)
	{
		Globals = new FGlobalVars();
	}
	return *Globals;
}

FInstanceCountingObject::FInstanceCountingObject(const TCHAR* InName, bool InLogConstruction)
	: Name(InName)
	, DoLog(InLogConstruction)
{
	IncrementStats();
}

FInstanceCountingObject::FInstanceCountingObject(const FInstanceCountingObject& RHS)
	: Name(RHS.Name)
	, DoLog(RHS.DoLog)
{
	IncrementStats();
}

FInstanceCountingObject::~FInstanceCountingObject()
{
	DecrementStats();
}

void FInstanceCountingObject::IncrementStats()
{
	int32 Count = 0;

	{
		FScopeLock Lock(&GetGlobals().Mutex);

		int32& RefCount = GetGlobals().InstanceCounts.FindOrAdd(Name);

		RefCount++;
		Count = RefCount;
	}

	if (DoLog)
	{
		UE_LOG(LogInstanceCount, Log, TEXT("Constructed %s at 0x%08" UPTRINT_X_FMT ", count=%d"), *Name.ToString(), this, Count);
	}
}

void FInstanceCountingObject::DecrementStats()
{
	int32 Count = 0;

	{
		check(Globals);
		FScopeLock Lock(&Globals->Mutex);

		int32& RefCount = Globals->InstanceCounts[Name];

		RefCount--;
		Count = RefCount;
	}

	if (DoLog)
	{
		UE_LOG(LogInstanceCount, Log, TEXT("Destructed %s at 0x%08" UPTRINT_X_FMT ", count=%d"), *Name.ToString(), this, Count);
	}
}

 int32	FInstanceCountingObject::GetInstanceCount(const TCHAR* Name)
{
	check(Globals);
	FScopeLock Lock(&Globals->Mutex);
	return Globals->InstanceCounts.FindOrAdd(FName(Name));
}

void FInstanceCountingObject::LogCounts(FOutputDevice& OutputDevice)
{
	if (Globals)
	{
		FScopeLock Lock(&Globals->Mutex);

		if (Globals->InstanceCounts.Num())
		{
			OutputDevice.Logf(TEXT("Manually tracked object counts:"));

			for (const auto& KP : Globals->InstanceCounts)
			{
				OutputDevice.Logf(TEXT("\t%s: %d instances"), *KP.Key.ToString(), KP.Value);
			}

			OutputDevice.Logf(TEXT(""));
		}
	}
}
