// Copyright Epic Games, Inc. All Rights Reserved.
#include "ProfilingDebugging/IoStoreTrace.h"

#if UE_TRACE_IOSTORE_ENABLED

#include "ProfilingDebugging/CallstackTrace.h"
#include "IO/IoDispatcherBackend.h"
#include "Trace/Trace.h"

// Capturing callstacks is disabled by default. 
// It is most useful for identifying locations in the code where an IoStore load may be missing metadata instrumentation.
// There is also a cpu & memory cost to capturing the callstacks too.
#if !defined(UE_TRACE_IOSTORE_CAPTURE_CALLSTACKS)
	#define UE_TRACE_IOSTORE_CAPTURE_CALLSTACKS 0
#endif


UE_TRACE_CHANNEL_DEFINE(IoStoreChannel, "I/O dispatcher requests, backend operations, and chunk loading.")
UE_TRACE_METADATA_EVENT_DEFINE(IoStoreTag)

UE_TRACE_EVENT_BEGIN(IoStore, BackendName, NoSync)
	UE_TRACE_EVENT_FIELD(uint64, BackendHandle)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, Name)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(IoStore, RequestCreate)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(uint64, RequestHandle)
	UE_TRACE_EVENT_FIELD(uint64, BatchHandle)
	UE_TRACE_EVENT_FIELD(uint32, ChunkIdHash)
	UE_TRACE_EVENT_FIELD(uint8,  ChunkType)
	UE_TRACE_EVENT_FIELD(uint32, CallstackId)
	UE_TRACE_EVENT_FIELD(uint64, Offset)
	UE_TRACE_EVENT_FIELD(uint64, Size)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(IoStore, RequestUnresolved)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(uint64, RequestHandle)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(IoStore, RequestStarted)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(uint64, RequestHandle)
	UE_TRACE_EVENT_FIELD(uint64, BackendHandle)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(IoStore, RequestCompleted)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(uint64, RequestHandle)
	UE_TRACE_EVENT_FIELD(uint64, Size)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(IoStore, RequestFailed)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(uint64, RequestHandle)
UE_TRACE_EVENT_END()

void FIoStoreTrace::BackendName(IIoDispatcherBackend* IoDispatcherBackend, const TCHAR* Name)
{
	UE_TRACE_LOG(IoStore, BackendName, IoStoreChannel)
		<< BackendName.BackendHandle((uint64)IoDispatcherBackend)
		<< BackendName.Name(Name)
		;
}

void FIoStoreTrace::RequestCreate(FIoBatch* IoBatch, FIoRequestImpl* IoRequestImpl)
{
	const uint32 CallstackId = 0;
	const uint32 ChunkIdHash = GetTypeHash(IoRequestImpl->ChunkId);

#if UE_TRACE_IOSTORE_CAPTURE_CALLSTACKS
	if (UE_TRACE_CHANNELEXPR_IS_ENABLED(IoStoreChannel))
	{
		CallstackId = CallstackTrace_GetCurrentId();
	}
#endif

	UE_TRACE_LOG(IoStore, RequestCreate, IoStoreChannel)
		<< RequestCreate.Cycle(FPlatformTime::Cycles64())
		<< RequestCreate.RequestHandle((uint64)IoRequestImpl)
		<< RequestCreate.BatchHandle((uint64)IoBatch)
		<< RequestCreate.ChunkIdHash(ChunkIdHash)
		<< RequestCreate.ChunkType((uint8)IoRequestImpl->ChunkId.GetChunkType())
		<< RequestCreate.CallstackId(CallstackId)
		<< RequestCreate.Offset(IoRequestImpl->Options.GetOffset())
		<< RequestCreate.Size(IoRequestImpl->Options.GetSize())
		;
}

void FIoStoreTrace::RequestUnresolved(FIoRequestImpl* IoRequestImpl)
{
	UE_TRACE_LOG(IoStore, RequestUnresolved, IoStoreChannel)
		<< RequestUnresolved.Cycle(FPlatformTime::Cycles64())
		<< RequestUnresolved.RequestHandle((uint64)IoRequestImpl)
		;
}

void FIoStoreTrace::RequestStarted(FIoRequestImpl* IoRequestImpl, IIoDispatcherBackend* IoDispatcherBackend)
{
	UE_TRACE_LOG(IoStore, RequestStarted, IoStoreChannel)
		<< RequestStarted.Cycle(FPlatformTime::Cycles64())
		<< RequestStarted.RequestHandle((uint64)IoRequestImpl)
		<< RequestStarted.BackendHandle((uint64)IoDispatcherBackend)
		;
}

void FIoStoreTrace::RequestCompleted(FIoRequestImpl* IoRequestImpl, uint64 Size)
{
	UE_TRACE_LOG(IoStore, RequestCompleted, IoStoreChannel)
		<< RequestCompleted.Cycle(FPlatformTime::Cycles64())
		<< RequestCompleted.RequestHandle((uint64)IoRequestImpl)
		<< RequestCompleted.Size(Size)
		;
}

void FIoStoreTrace::RequestFailed(FIoRequestImpl* IoRequestImpl)
{
	UE_TRACE_LOG(IoStore, RequestFailed, IoStoreChannel)
		<< RequestFailed.Cycle(FPlatformTime::Cycles64())
		<< RequestFailed.RequestHandle((uint64)IoRequestImpl)
		;
}


#endif // UE_TRACE_IOSTORE_ENABLED
