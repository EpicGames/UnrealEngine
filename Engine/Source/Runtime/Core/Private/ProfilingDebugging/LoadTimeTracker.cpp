// Copyright Epic Games, Inc. All Rights Reserved.

/**
* Here are a number of profiling helper functions so we do not have to duplicate a lot of the glue
* code everywhere.  And we can have consistent naming for all our files.
*
*/

// Core includes.
#include "ProfilingDebugging/LoadTimeTracker.h"
#include "Misc/Parse.h"
#include "Misc/CommandLine.h"
#include "HAL/IConsoleManager.h"

double FScopedLoadTimeAccumulatorTimer::DummyTimer = 0.0;

FScopedLoadTimeAccumulatorTimer::FScopedLoadTimeAccumulatorTimer(const FName& InTimerName, const FName& InInstanceName)
	: FScopedDurationTimer(FLoadTimeTracker::Get().IsAccumulating() ? FLoadTimeTracker::Get().GetScopeTimeAccumulator(InTimerName, InInstanceName) : DummyTimer)
{}

FLoadTimeTracker::FLoadTimeTracker()
{
	ResetRawLoadTimes();
	bAccumulating = false;
}

FLoadTimeTracker& FLoadTimeTracker::Get()
{
	static FLoadTimeTracker Singleton;
	return Singleton;
}

void FLoadTimeTracker::ReportScopeTime(double ScopeTime, const FName ScopeLabel)
{
	check(IsInGameThread());
	TArray<double>& LoadTimes = TimeInfo.FindOrAdd(ScopeLabel);
	LoadTimes.Add(ScopeTime);
}

double& FLoadTimeTracker::GetScopeTimeAccumulator(const FName& ScopeLabel, const FName& ScopeInstance)
{
	check(IsInGameThread());
	FAccumulatorTracker& Tracker = AccumulatedTimeInfo.FindOrAdd(ScopeLabel);
	FTimeAndCount& TimeAndCount = Tracker.TimeInfo.FindOrAdd(ScopeInstance);
	TimeAndCount.Count++;
	return TimeAndCount.Time;
}

void FLoadTimeTracker::DumpHighLevelLoadTimes() const
{
	double TotalTime = 0.0;
	UE_LOGF(LogLoad, Log, "------------- Load times -------------");
	for(auto Itr = TimeInfo.CreateConstIterator(); Itr; ++Itr)
	{
		const FString KeyName = Itr.Key().ToString();
		const TArray<double>& LoadTimes = Itr.Value();
		if(LoadTimes.Num() == 1)
		{
			TotalTime += Itr.Value()[0];
			UE_LOGF(LogLoad, Log, "%ls: %f", *KeyName, Itr.Value()[0]);
		}
		else
		{
			double InnerTotal = 0.0;
			for(int Index = 0; Index < LoadTimes.Num(); ++Index)
			{
				InnerTotal += Itr.Value()[Index];
				UE_LOGF(LogLoad, Log, "%ls[%d]: %f", *KeyName, Index, LoadTimes[Index]);
			}

			UE_LOGF(LogLoad, Log, "    Sub-Total: %f", InnerTotal);

			TotalTime += InnerTotal;
		}
		
	}
	UE_LOGF(LogLoad, Log, "------------- ---------- -------------");
	UE_LOGF(LogLoad, Log, "Total Load times: %f", TotalTime);
}

void FLoadTimeTracker::ResetHighLevelLoadTimes()
{
	static bool bActuallyReset = !FParse::Param(FCommandLine::Get(), TEXT("NoLoadTrackClear"));
	if(bActuallyReset)
	{
		TimeInfo.Reset();
	}
}

void FLoadTimeTracker::DumpRawLoadTimes() const
{
#if ENABLE_LOADTIME_RAW_TIMINGS
	UE_LOGF(LogStreaming, Display, "-------------------------------------------------");
	UE_LOGF(LogStreaming, Display, "Async Loading Stats");
	UE_LOGF(LogStreaming, Display, "-------------------------------------------------");
	UE_LOGF(LogStreaming, Display, "AsyncLoadingTime: %f", AsyncLoadingTime);
	UE_LOGF(LogStreaming, Display, "CreateAsyncPackagesFromQueueTime: %f", CreateAsyncPackagesFromQueueTime);
	UE_LOGF(LogStreaming, Display, "ProcessAsyncLoadingTime: %f", ProcessAsyncLoadingTime);
	UE_LOGF(LogStreaming, Display, "ProcessLoadedPackagesTime: %f", ProcessLoadedPackagesTime);
	//UE_LOGF(LogStreaming, Display, "SerializeTaggedPropertiesTime: %f", SerializeTaggedPropertiesTime);
	UE_LOGF(LogStreaming, Display, "CreateLinkerTime: %f", CreateLinkerTime);
	UE_LOGF(LogStreaming, Display, "FinishLinkerTime: %f", FinishLinkerTime);
	UE_LOGF(LogStreaming, Display, "CreateImportsTime: %f", CreateImportsTime);
	UE_LOGF(LogStreaming, Display, "CreateExportsTime: %f", CreateExportsTime);
	UE_LOGF(LogStreaming, Display, "PreLoadObjectsTime: %f", PreLoadObjectsTime);
	UE_LOGF(LogStreaming, Display, "PostLoadObjectsTime: %f", PostLoadObjectsTime);
	UE_LOGF(LogStreaming, Display, "PostLoadDeferredObjectsTime: %f", PostLoadDeferredObjectsTime);
	UE_LOGF(LogStreaming, Display, "FinishObjectsTime: %f", FinishObjectsTime);
	UE_LOGF(LogStreaming, Display, "MaterialPostLoad: %f", MaterialPostLoad);
	UE_LOGF(LogStreaming, Display, "MaterialInstancePostLoad: %f", MaterialInstancePostLoad);
	UE_LOGF(LogStreaming, Display, "SerializeInlineShaderMaps: %f", SerializeInlineShaderMaps);
	UE_LOGF(LogStreaming, Display, "MaterialSerializeTime: %f", MaterialSerializeTime);
	UE_LOGF(LogStreaming, Display, "MaterialInstanceSerializeTime: %f", MaterialInstanceSerializeTime);
	UE_LOGF(LogStreaming, Display, "");
	UE_LOGF(LogStreaming, Display, "LinkerLoad_CreateLoader: %f", LinkerLoad_CreateLoader);
	UE_LOGF(LogStreaming, Display, "LinkerLoad_SerializePackageFileSummary: %f", LinkerLoad_SerializePackageFileSummary);
	UE_LOGF(LogStreaming, Display, "LinkerLoad_SerializeNameMap: %f", LinkerLoad_SerializeNameMap);
	UE_LOGF(LogStreaming, Display, "\tProcessingEntries: %f", LinkerLoad_SerializeNameMap_ProcessingEntries);
	UE_LOGF(LogStreaming, Display, "LinkerLoad_SerializeGatherableTextDataMap: %f", LinkerLoad_SerializeGatherableTextDataMap);
	UE_LOGF(LogStreaming, Display, "LinkerLoad_SerializeImportMap: %f", LinkerLoad_SerializeImportMap);
	UE_LOGF(LogStreaming, Display, "LinkerLoad_SerializeExportMap: %f", LinkerLoad_SerializeExportMap);
	UE_LOGF(LogStreaming, Display, "LinkerLoad_FixupImportMap: %f", LinkerLoad_FixupImportMap);
	UE_LOGF(LogStreaming, Display, "LinkerLoad_FixupExportMap: %f", LinkerLoad_FixupExportMap);
	UE_LOGF(LogStreaming, Display, "LinkerLoad_SerializeMetaData: %f", LinkerLoad_SerializeMetaData);
	UE_LOGF(LogStreaming, Display, "LinkerLoad_SerializeDependsMap: %f", LinkerLoad_SerializeDependsMap);
	UE_LOGF(LogStreaming, Display, "LinkerLoad_SerializePreloadDependencies: %f", LinkerLoad_SerializePreloadDependencies);
	UE_LOGF(LogStreaming, Display, "LinkerLoad_CreateExportHash: %f", LinkerLoad_CreateExportHash);
	UE_LOGF(LogStreaming, Display, "LinkerLoad_FindExistingExports: %f", LinkerLoad_FindExistingExports);
	UE_LOGF(LogStreaming, Display, "LinkerLoad_FinalizeCreation: %f", LinkerLoad_FinalizeCreation);

	UE_LOGF(LogStreaming, Display, "Package_FinishLinker: %f", Package_FinishLinker);
	UE_LOGF(LogStreaming, Display, "Package_LoadImports: %f", Package_LoadImports);
	UE_LOGF(LogStreaming, Display, "Package_CreateImports: %f", Package_CreateImports);
	UE_LOGF(LogStreaming, Display, "Package_CreateLinker: %f", Package_CreateLinker);
	UE_LOGF(LogStreaming, Display, "Package_CreateExports: %f", Package_CreateExports);
	UE_LOGF(LogStreaming, Display, "Package_PreLoadObjects: %f", Package_PreLoadObjects);
	UE_LOGF(LogStreaming, Display, "Package_ExternalReadDependencies: %f", Package_ExternalReadDependencies);
	UE_LOGF(LogStreaming, Display, "Package_PostLoadObjects: %f", Package_PostLoadObjects);
	UE_LOGF(LogStreaming, Display, "Package_Tick: %f", Package_Tick);
	UE_LOGF(LogStreaming, Display, "Package_CreateAsyncPackagesFromQueue: %f", Package_CreateAsyncPackagesFromQueue);
	UE_LOGF(LogStreaming, Display, "Package_EventIOWait: %f", Package_EventIOWait);

	UE_LOGF(LogStreaming, Display, "TickAsyncLoading_ProcessLoadedPackages: %f", TickAsyncLoading_ProcessLoadedPackages);

	UE_LOGF(LogStreaming, Display, "Package_Temp1: %f", Package_Temp1);
	UE_LOGF(LogStreaming, Display, "Package_Temp2: %f", Package_Temp2);
	UE_LOGF(LogStreaming, Display, "Package_Temp3: %f", Package_Temp3);
	UE_LOGF(LogStreaming, Display, "Package_Temp4: %f", Package_Temp4);

	UE_LOGF(LogStreaming, Display, "Graph_AddNode: %f     %u", Graph_AddNode, Graph_AddNodeCnt);
	UE_LOGF(LogStreaming, Display, "Graph_AddArc: %f     %u", Graph_AddArc, Graph_AddArcCnt);
	UE_LOGF(LogStreaming, Display, "Graph_RemoveNode: %f     %u", Graph_RemoveNode, Graph_RemoveNodeCnt);
	UE_LOGF(LogStreaming, Display, "Graph_RemoveNodeFire: %f     %u", Graph_RemoveNodeFire, Graph_RemoveNodeFireCnt);
	UE_LOGF(LogStreaming, Display, "Graph_DoneAddingPrerequistesFireIfNone: %f     %u", Graph_DoneAddingPrerequistesFireIfNone, Graph_DoneAddingPrerequistesFireIfNoneCnt);
	UE_LOGF(LogStreaming, Display, "Graph_DoneAddingPrerequistesFireIfNoneFire: %f     %u", Graph_DoneAddingPrerequistesFireIfNoneFire, Graph_DoneAddingPrerequistesFireIfNoneFireCnt);
	UE_LOGF(LogStreaming, Display, "Graph_Misc: %f     %u", Graph_Misc, Graph_MiscCnt);


	UE_LOGF(LogStreaming, Display, "TickAsyncLoading_ProcessLoadedPackages: %f", TickAsyncLoading_ProcessLoadedPackages);
	UE_LOGF(LogStreaming, Display, "LinkerLoad_SerializeNameMap_ProcessingEntries: %f", LinkerLoad_SerializeNameMap_ProcessingEntries);
	UE_LOGF(LogStreaming, Display, "FFileCacheHandle_AcquireSlotAndReadLine: %f", FFileCacheHandle_AcquireSlotAndReadLine);
	UE_LOGF(LogStreaming, Display, "FFileCacheHandle_PreloadData: %f", FFileCacheHandle_PreloadData);
	UE_LOGF(LogStreaming, Display, "FFileCacheHandle_ReadData: %f", FFileCacheHandle_ReadData);
	UE_LOGF(LogStreaming, Display, "FTypeLayoutDesc_Find: %f", FTypeLayoutDesc_Find);
	UE_LOGF(LogStreaming, Display, "FMemoryImageResult_ApplyPatchesFromArchive: %f", FMemoryImageResult_ApplyPatchesFromArchive);
	UE_LOGF(LogStreaming, Display, "LoadImports_Event: %f", LoadImports_Event);
	UE_LOGF(LogStreaming, Display, "StartPrecacheRequests: %f", StartPrecacheRequests);
	UE_LOGF(LogStreaming, Display, "MakeNextPrecacheRequestCurrent: %f", MakeNextPrecacheRequestCurrent);
	UE_LOGF(LogStreaming, Display, "FlushPrecacheBuffer: %f", FlushPrecacheBuffer);
	UE_LOGF(LogStreaming, Display, "ProcessImportsAndExports_Event: %f", ProcessImportsAndExports_Event);
	UE_LOGF(LogStreaming, Display, "CreateLinker_CreatePackage: %f", CreateLinker_CreatePackage);
	UE_LOGF(LogStreaming, Display, "CreateLinker_SetFlags: %f", CreateLinker_SetFlags);
	UE_LOGF(LogStreaming, Display, "CreateLinker_FindLinker: %f", CreateLinker_FindLinker);
	UE_LOGF(LogStreaming, Display, "CreateLinker_GetRedirectedName: %f", CreateLinker_GetRedirectedName);
	UE_LOGF(LogStreaming, Display, "CreateLinker_MassagePath: %f", CreateLinker_MassagePath);
	UE_LOGF(LogStreaming, Display, "CreateLinker_DoesExist: %f", CreateLinker_DoesExist);
	UE_LOGF(LogStreaming, Display, "CreateLinker_MissingPackage: %f", CreateLinker_MissingPackage);
	UE_LOGF(LogStreaming, Display, "CreateLinker_CreateLinkerAsync: %f", CreateLinker_CreateLinkerAsync);
	UE_LOGF(LogStreaming, Display, "FPackageName_DoesPackageExist: %f", FPackageName_DoesPackageExist);
	UE_LOGF(LogStreaming, Display, "PreLoadAndSerialize: %f", PreLoadAndSerialize);
	UE_LOGF(LogStreaming, Display, "PostLoad: %f", PostLoad);
	UE_LOGF(LogStreaming, Display, "LinkerLoad_ReconstructImportAndExportMap: %f", LinkerLoad_ReconstructImportAndExportMap);
	UE_LOGF(LogStreaming, Display, "LinkerLoad_PopulateInstancingContext: %f", LinkerLoad_PopulateInstancingContext);
	UE_LOGF(LogStreaming, Display, "LinkerLoad_VerifyImportInner: %f", LinkerLoad_VerifyImportInner);
	UE_LOGF(LogStreaming, Display, "LinkerLoad_LoadAllObjects: %f", LinkerLoad_LoadAllObjects);
	UE_LOGF(LogStreaming, Display, "UObject_Serialize: %f", UObject_Serialize);
	UE_LOGF(LogStreaming, Display, "BulkData_Serialize: %f", BulkData_Serialize);
	UE_LOGF(LogStreaming, Display, "BulkData_SerializeBulkData: %f", BulkData_SerializeBulkData);
	UE_LOGF(LogStreaming, Display, "EndLoad: %f", EndLoad);
	UE_LOGF(LogStreaming, Display, "FTextureReference_InitRHI: %f", FTextureReference_InitRHI);
	UE_LOGF(LogStreaming, Display, "FShaderMapPointerTable_LoadFromArchive: %f", FShaderMapPointerTable_LoadFromArchive);
	UE_LOGF(LogStreaming, Display, "FShaderLibraryInstance_PreloadShaderMap: %f", FShaderLibraryInstance_PreloadShaderMap);
	UE_LOGF(LogStreaming, Display, "LoadShaderResource_Internal: %f", LoadShaderResource_Internal);
	UE_LOGF(LogStreaming, Display, "LoadShaderResource_AddOrDeleteResource: %f", LoadShaderResource_AddOrDeleteResource);
	UE_LOGF(LogStreaming, Display, "FShaderCodeLibrary_LoadResource: %f", FShaderCodeLibrary_LoadResource);
	UE_LOGF(LogStreaming, Display, "FMaterialShaderMapId_Serialize: %f", FMaterialShaderMapId_Serialize);
	UE_LOGF(LogStreaming, Display, "FMaterialShaderMapLayoutCache_CreateLayout: %f", FMaterialShaderMapLayoutCache_CreateLayout);
	UE_LOGF(LogStreaming, Display, "FMaterialShaderMap_IsComplete: %f", FMaterialShaderMap_IsComplete);
	UE_LOGF(LogStreaming, Display, "FMaterialShaderMap_Serialize: %f", FMaterialShaderMap_Serialize);
	UE_LOGF(LogStreaming, Display, "FMaterialResourceProxyReader_Initialize: %f", FMaterialResourceProxyReader_Initialize);
	UE_LOGF(LogStreaming, Display, "FSkeletalMeshVertexClothBuffer_InitRHI: %f", FSkeletalMeshVertexClothBuffer_InitRHI);
	UE_LOGF(LogStreaming, Display, "FSkinWeightVertexBuffer_InitRHI: %f", FSkinWeightVertexBuffer_InitRHI);
	UE_LOGF(LogStreaming, Display, "FStaticMeshVertexBuffer_InitRHI: %f", FStaticMeshVertexBuffer_InitRHI);
	UE_LOGF(LogStreaming, Display, "FStreamableTextureResource_InitRHI: %f", FStreamableTextureResource_InitRHI);
	UE_LOGF(LogStreaming, Display, "FShaderLibraryInstance_PreloadShader: %f", FShaderLibraryInstance_PreloadShader);
	UE_LOGF(LogStreaming, Display, "FShaderMapResource_SharedCode_InitRHI: %f", FShaderMapResource_SharedCode_InitRHI);
	UE_LOGF(LogStreaming, Display, "FStaticMeshInstanceBuffer_InitRHI: %f", FStaticMeshInstanceBuffer_InitRHI);
	UE_LOGF(LogStreaming, Display, "FInstancedStaticMeshVertexFactory_InitRHI: %f", FInstancedStaticMeshVertexFactory_InitRHI);
	UE_LOGF(LogStreaming, Display, "FLocalVertexFactory_InitRHI: %f", FLocalVertexFactory_InitRHI);
	UE_LOGF(LogStreaming, Display, "FLocalVertexFactory_InitRHI_CreateLocalVFUniformBuffer: %f", FLocalVertexFactory_InitRHI_CreateLocalVFUniformBuffer);
	UE_LOGF(LogStreaming, Display, "FSinglePrimitiveStructuredBuffer_InitRHI: %f", FSinglePrimitiveStructuredBuffer_InitRHI);
	UE_LOGF(LogStreaming, Display, "FColorVertexBuffer_InitRHI: %f", FColorVertexBuffer_InitRHI);
	UE_LOGF(LogStreaming, Display, "FFMorphTargetVertexInfoBuffers_InitRHI: %f", FFMorphTargetVertexInfoBuffers_InitRHI);
	UE_LOGF(LogStreaming, Display, "FSlateTexture2DRHIRef_InitDynamicRHI: %f", FSlateTexture2DRHIRef_InitDynamicRHI);
	UE_LOGF(LogStreaming, Display, "FLightmapResourceCluster_InitRHI: %f", FLightmapResourceCluster_InitRHI);
	UE_LOGF(LogStreaming, Display, "UMaterialExpression_Serialize: %f", UMaterialExpression_Serialize);
	UE_LOGF(LogStreaming, Display, "UMaterialExpression_PostLoad: %f", UMaterialExpression_PostLoad);
	UE_LOGF(LogStreaming, Display, "FSlateTextureRenderTarget2DResource_InitDynamicRHI: %f", FSlateTextureRenderTarget2DResource_InitDynamicRHI);
	UE_LOGF(LogStreaming, Display, "VerifyGlobalShaders: %f", VerifyGlobalShaders);
	UE_LOGF(LogStreaming, Display, "FLandscapeVertexBuffer_InitRHI: %f", FLandscapeVertexBuffer_InitRHI);


	UE_LOGF(LogStreaming, Display, "-------------------------------------------------");

#endif

}

void FLoadTimeTracker::ResetRawLoadTimes()
{
#if ENABLE_LOADTIME_RAW_TIMINGS
	CreateAsyncPackagesFromQueueTime = 0.0;
	ProcessAsyncLoadingTime = 0.0;
	ProcessLoadedPackagesTime = 0.0;
	SerializeTaggedPropertiesTime = 0.0;
	CreateLinkerTime = 0.0;
	FinishLinkerTime = 0.0;
	CreateImportsTime = 0.0;
	CreateExportsTime = 0.0;
	PreLoadObjectsTime = 0.0;
	PostLoadObjectsTime = 0.0;
	PostLoadDeferredObjectsTime = 0.0;
	FinishObjectsTime = 0.0;
	MaterialPostLoad = 0.0;
	MaterialInstancePostLoad = 0.0;
	SerializeInlineShaderMaps = 0.0;
	MaterialSerializeTime = 0.0;
	MaterialInstanceSerializeTime = 0.0;
	AsyncLoadingTime = 0.0;
	CreateMetaDataTime = 0.0;

	LinkerLoad_CreateLoader = 0.0;
	LinkerLoad_SerializePackageFileSummary = 0.0;
	LinkerLoad_SerializeNameMap = 0.0;
	LinkerLoad_SerializeGatherableTextDataMap = 0.0;
	LinkerLoad_SerializeImportMap = 0.0;
	LinkerLoad_SerializeExportMap = 0.0;
	LinkerLoad_FixupImportMap = 0.0;
	LinkerLoad_FixupExportMap = 0.0;
	LinkerLoad_SerializeMetaData = 0.0;
	LinkerLoad_SerializeDependsMap = 0.0;
	LinkerLoad_SerializePreloadDependencies = 0.0;
	LinkerLoad_CreateExportHash = 0.0;
	LinkerLoad_FindExistingExports = 0.0;
	LinkerLoad_FinalizeCreation = 0.0;

	Package_FinishLinker = 0.0;
	Package_LoadImports = 0.0;
	Package_CreateImports = 0.0;
	Package_CreateLinker = 0.0;
	Package_CreateExports = 0.0;
	Package_PreLoadObjects = 0.0;
	Package_ExternalReadDependencies = 0.0;
	Package_PostLoadObjects = 0.0;
	Package_Tick = 0.0;
	Package_CreateAsyncPackagesFromQueue = 0.0;
	Package_CreateMetaData = 0.0;
	Package_EventIOWait = 0.0;

	Package_Temp1 = 0.0;
	Package_Temp2 = 0.0;
	Package_Temp3 = 0.0;
	Package_Temp4 = 0.0;

	Graph_AddNode = 0.0;
	Graph_AddNodeCnt = 0;

	Graph_AddArc = 0.0;
	Graph_AddArcCnt = 0;

	Graph_RemoveNode = 0.0;
	Graph_RemoveNodeCnt = 0;

	Graph_RemoveNodeFire = 0.0;
	Graph_RemoveNodeFireCnt = 0;

	Graph_DoneAddingPrerequistesFireIfNone = 0.0;
	Graph_DoneAddingPrerequistesFireIfNoneCnt = 0;

	Graph_DoneAddingPrerequistesFireIfNoneFire = 0.0;
	Graph_DoneAddingPrerequistesFireIfNoneFireCnt = 0;

	Graph_Misc = 0.0;
	Graph_MiscCnt = 0;

	TickAsyncLoading_ProcessLoadedPackages = 0.0;

	LinkerLoad_SerializeNameMap_ProcessingEntries = 0.0;

	FFileCacheHandle_AcquireSlotAndReadLine = 0.0;
	FFileCacheHandle_PreloadData = 0.0;
	FFileCacheHandle_ReadData = 0.0;

	FTypeLayoutDesc_Find = 0.0;

	FMemoryImageResult_ApplyPatchesFromArchive = 0.0;
	LoadImports_Event = 0.0;
	StartPrecacheRequests = 0.0;
	MakeNextPrecacheRequestCurrent = 0.0;
	FlushPrecacheBuffer = 0.0;
	ProcessImportsAndExports_Event = 0.0;
	CreateLinker_CreatePackage = 0.0;
	CreateLinker_SetFlags = 0.0;
	CreateLinker_FindLinker = 0.0;
	CreateLinker_GetRedirectedName = 0.0;
	CreateLinker_MassagePath = 0.0;
	CreateLinker_DoesExist = 0.0;
	CreateLinker_MissingPackage = 0.0;
	CreateLinker_CreateLinkerAsync = 0.0;
	FPackageName_DoesPackageExist = 0.0;
	PreLoadAndSerialize = 0.0;
	PostLoad = 0.0;
	LinkerLoad_ReconstructImportAndExportMap = 0.0;
	LinkerLoad_PopulateInstancingContext = 0.0;
	LinkerLoad_VerifyImportInner = 0.0;
	LinkerLoad_LoadAllObjects = 0.0;
	UObject_Serialize = 0.0;
	BulkData_Serialize = 0.0;
	BulkData_SerializeBulkData = 0.0;
	EndLoad = 0.0;
	FTextureReference_InitRHI = 0.0;
	FShaderMapPointerTable_LoadFromArchive = 0.0;
	FShaderLibraryInstance_PreloadShaderMap = 0.0;
	LoadShaderResource_Internal = 0.0;
	LoadShaderResource_AddOrDeleteResource = 0.0;
	FShaderCodeLibrary_LoadResource = 0.0;
	FMaterialShaderMapId_Serialize = 0.0;
	FMaterialShaderMapLayoutCache_CreateLayout = 0.0;
	FMaterialShaderMap_IsComplete = 0.0;
	FMaterialShaderMap_Serialize = 0.0;
	FMaterialResourceProxyReader_Initialize = 0.0;
	FSkeletalMeshVertexClothBuffer_InitRHI = 0.0;
	FSkinWeightVertexBuffer_InitRHI = 0.0;
	FStaticMeshVertexBuffer_InitRHI = 0.0;
	FStreamableTextureResource_InitRHI = 0.0;
	FShaderLibraryInstance_PreloadShader = 0.0;
	FShaderMapResource_SharedCode_InitRHI = 0.0;
	FStaticMeshInstanceBuffer_InitRHI = 0.0;
	FInstancedStaticMeshVertexFactory_InitRHI = 0.0;
	FLocalVertexFactory_InitRHI = 0.0;
	FLocalVertexFactory_InitRHI_CreateLocalVFUniformBuffer = 0.0;
	FSinglePrimitiveStructuredBuffer_InitRHI = 0.0;
	FColorVertexBuffer_InitRHI = 0.0;
	FFMorphTargetVertexInfoBuffers_InitRHI = 0.0;
	FSlateTexture2DRHIRef_InitDynamicRHI = 0.0;
	FLightmapResourceCluster_InitRHI = 0.0;
	UMaterialExpression_Serialize = 0.0;
	UMaterialExpression_PostLoad = 0.0;
	FSlateTextureRenderTarget2DResource_InitDynamicRHI = 0.0;
	VerifyGlobalShaders = 0.0;
	FLandscapeVertexBuffer_InitRHI = 0.0;

#endif

}

void FLoadTimeTracker::StartAccumulatedLoadTimes()
{
	bAccumulating = true;
	AccumulatedTimeInfo.Empty();
}

void FLoadTimeTracker::StopAccumulatedLoadTimes()
{
	bAccumulating = false;

	UE_LOGF(LogLoad, Log, "------------- Accumulated Load times -------------");
	
	for(auto Itr0 = AccumulatedTimeInfo.CreateConstIterator(); Itr0; ++Itr0)
	{
		double TotalTime = 0.0;
		uint64 TotalCount = 0;

		const FString KeyName = Itr0.Key().ToString();
		UE_LOGF(LogLoad, Log, "------------- %ls Times ------------", *KeyName);
		UE_LOGF(LogLoad, Log, "Name Time Count");

		for(auto Itr1 = Itr0.Value().TimeInfo.CreateConstIterator(); Itr1; ++Itr1)
		{
			const FString InstanceName = Itr1.Key().ToString();
			const double& LoadTime = Itr1.Value().Time;
			const uint64& Count = Itr1.Value().Count;

			TotalTime += LoadTime;
			TotalCount += Count;
			UE_LOGF(LogLoad, Log, "%ls %f %llu", *InstanceName, LoadTime, Count);
		}

		UE_LOGF(LogLoad, Log, "Total%ls %f %llu", *KeyName, TotalTime, TotalCount);
		UE_LOGF(LogLoad, Log, "------------------------------------");
	}
}

static FAutoConsoleCommand LoadTimerDumpCmd(
	TEXT("LoadTimes.DumpTracking"),
	TEXT("Dump high level load times being tracked"),
	FConsoleCommandDelegate::CreateStatic(&FLoadTimeTracker::DumpHighLevelLoadTimesStatic)
	);
static FAutoConsoleCommand LoadTimerDumpLowCmd(
	TEXT("LoadTimes.DumpTrackingLow"),
	TEXT("Dump low level load times being tracked"),
	FConsoleCommandDelegate::CreateStatic(&FLoadTimeTracker::DumpRawLoadTimesStatic)
	);

static FAutoConsoleCommand LoadTimerResetCmd(
	TEXT("LoadTimes.ResetTracking"),
	TEXT("Reset load time tracking"),
	FConsoleCommandDelegate::CreateStatic(&FLoadTimeTracker::ResetRawLoadTimesStatic)
	);

static FAutoConsoleCommand AccumulatorTimerStartCmd(
	TEXT("LoadTimes.StartAccumulating"),
	TEXT("Starts capturing fine-grained accumulated load time data"),
	FConsoleCommandDelegate::CreateStatic(&FLoadTimeTracker::StartAccumulatedLoadTimesStatic)
	);

static FAutoConsoleCommand AccumulatorTimerStopCmd(
	TEXT("LoadTimes.StopAccumulating"),
	TEXT("Stops capturing fine-grained accumulated load time data and dump the results"),
	FConsoleCommandDelegate::CreateStatic(&FLoadTimeTracker::StopAccumulatedLoadTimesStatic)
	);
