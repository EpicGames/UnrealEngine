// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProfilingDebugging/MemoryTrace.h"
#include "Containers/StringView.h"
#include "HAL/PlatformTime.h"
#include "ProfilingDebugging/CallstackTrace.h"
#include "ProfilingDebugging/MetadataTrace.h"
#include "ProfilingDebugging/TagTrace.h"
#include "ProfilingDebugging/TraceMalloc.h"
#include "Trace/Detail/EventNode.h"
#include "Trace/Detail/LogScope.inl"
#include "Trace/Detail/Important/ImportantLogScope.inl"
#include "Trace/Trace.h"

#if UE_TRACE_ENABLED
UE_TRACE_CHANNEL_DEFINE(MemAllocChannel, "Traces memory allocations and displays them in the Memory Insights Profiler. Optional callstack data is associated using Module + Callstack channels. \
Optional asset metadata is associated using Metadata + AssetMetadata channels.", true)
#endif

#if UE_MEMORY_TRACE_ENABLED

////////////////////////////////////////////////////////////////////////////////
void MemoryTrace_InitTags(FMalloc*);
void MemoryTrace_EnableTracePump();

#if PLATFORM_IOS || PLATFORM_MAC
void MemoryTrace_InitMemoryProbe();
#else
void MemoryTrace_InitMemoryProbe() {}
#endif

////////////////////////////////////////////////////////////////////////////////
namespace
{
	// Controls how often time markers are emitted (default: every 4095 allocations).
	constexpr uint32 MarkerSamplePeriod = (4 << 10) - 1;

	// Number of shifted bits to SizeLower
	constexpr uint32 SizeShift = 3;

	// Counter to track when time marker is emitted
	std::atomic<uint32> GMarkerCounter(0);

	// If enabled also pumps the Trace system itself. Used on process shutdown
	// when worker thread has been killed, but memory events still occurs.
	bool GDoPumpTrace;

	// Temporarily disables any internal operation that causes allocations. Used to
	// avoid recursive behaviour when memory tracing needs to allocate memory through
	// TraceMalloc.
	thread_local bool GDoNotAllocateInTrace;

	// Set on initialization; on some platforms we hook allocator functions very early
	// before Trace has the ability to allocate memory.
	bool GTraceAllowed;
}

////////////////////////////////////////////////////////////////////////////////
namespace UE {
namespace Trace {
	TRACELOG_API void Update();
} // namespace Trace
} // namespace UE

////////////////////////////////////////////////////////////////////////////////
UE_TRACE_EVENT_BEGIN(Memory, Init, NoSync|Important)
	UE_TRACE_EVENT_FIELD(uint64, PageSize) // new in UE 5.5
	UE_TRACE_EVENT_FIELD(uint32, MarkerPeriod)
	UE_TRACE_EVENT_FIELD(uint8, Version)
	UE_TRACE_EVENT_FIELD(uint8, MinAlignment)
	UE_TRACE_EVENT_FIELD(uint8, SizeShift)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Memory, Marker)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Memory, Alloc)
	UE_TRACE_EVENT_FIELD(uint64, Address)
	UE_TRACE_EVENT_FIELD(uint32, CallstackId)
	UE_TRACE_EVENT_FIELD(uint32, Size)
	UE_TRACE_EVENT_FIELD(uint8, AlignmentPow2_SizeLower)
	UE_TRACE_EVENT_FIELD(uint8, RootHeap)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Memory, AllocSystem)
	UE_TRACE_EVENT_FIELD(uint64, Address)
	UE_TRACE_EVENT_FIELD(uint32, CallstackId)
	UE_TRACE_EVENT_FIELD(uint32, Size)
	UE_TRACE_EVENT_FIELD(uint8, AlignmentPow2_SizeLower)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Memory, AllocVideo)
	UE_TRACE_EVENT_FIELD(uint64, Address)
	UE_TRACE_EVENT_FIELD(uint32, CallstackId)
	UE_TRACE_EVENT_FIELD(uint32, Size)
	UE_TRACE_EVENT_FIELD(uint8, AlignmentPow2_SizeLower)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Memory, Free)
	UE_TRACE_EVENT_FIELD(uint64, Address)
	UE_TRACE_EVENT_FIELD(uint32, CallstackId)
	UE_TRACE_EVENT_FIELD(uint8, RootHeap)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Memory, FreeSystem)
	UE_TRACE_EVENT_FIELD(uint64, Address)
	UE_TRACE_EVENT_FIELD(uint32, CallstackId)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Memory, FreeVideo)
	UE_TRACE_EVENT_FIELD(uint64, Address)
	UE_TRACE_EVENT_FIELD(uint32, CallstackId)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Memory, ReallocAlloc)
	UE_TRACE_EVENT_FIELD(uint64, Address)
	UE_TRACE_EVENT_FIELD(uint32, CallstackId)
	UE_TRACE_EVENT_FIELD(uint32, Size)
	UE_TRACE_EVENT_FIELD(uint8, AlignmentPow2_SizeLower)
	UE_TRACE_EVENT_FIELD(uint8, RootHeap)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Memory, ReallocAllocSystem)
	UE_TRACE_EVENT_FIELD(uint64, Address)
	UE_TRACE_EVENT_FIELD(uint32, CallstackId)
	UE_TRACE_EVENT_FIELD(uint32, Size)
	UE_TRACE_EVENT_FIELD(uint8, AlignmentPow2_SizeLower)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Memory, ReallocFree)
	UE_TRACE_EVENT_FIELD(uint64, Address)
	UE_TRACE_EVENT_FIELD(uint32, CallstackId)
	UE_TRACE_EVENT_FIELD(uint8, RootHeap)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Memory, ReallocFreeSystem)
	UE_TRACE_EVENT_FIELD(uint64, Address)
	UE_TRACE_EVENT_FIELD(uint32, CallstackId)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Memory, UpdateAlloc) // added in UE 5.6
	UE_TRACE_EVENT_FIELD(uint64, Address)
	UE_TRACE_EVENT_FIELD(uint32, CallstackId)
	UE_TRACE_EVENT_FIELD(uint8, RootHeap)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Memory, MemorySwapOp) // added in UE 5.5
	UE_TRACE_EVENT_FIELD(uint64, Address) // page fault real address
	UE_TRACE_EVENT_FIELD(uint32, CallstackId)
	UE_TRACE_EVENT_FIELD(uint32, CompressedSize)
	UE_TRACE_EVENT_FIELD(uint8,  SwapOp)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Memory, HeapSpec, NoSync|Important)
	UE_TRACE_EVENT_FIELD(HeapId, Id)
	UE_TRACE_EVENT_FIELD(HeapId, ParentId)
	UE_TRACE_EVENT_FIELD(uint16, Flags)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, Name)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Memory, HeapMarkAlloc)
	UE_TRACE_EVENT_FIELD(uint64, Address)
	UE_TRACE_EVENT_FIELD(uint32, CallstackId)
	UE_TRACE_EVENT_FIELD(uint16, Flags)
	UE_TRACE_EVENT_FIELD(HeapId, Heap)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Memory, HeapUnmarkAlloc)
	UE_TRACE_EVENT_FIELD(uint64, Address)
	UE_TRACE_EVENT_FIELD(uint32, CallstackId)
	UE_TRACE_EVENT_FIELD(HeapId, Heap)
UE_TRACE_EVENT_END()

// If the layout of the above events is changed, bump this version number.
// version 1: Initial version (UE 5.0, UE 5.1)
// version 2: Added CallstackId for Free events and also for HeapMarkAlloc, HeapUnmarkAlloc events (UE 5.2).
constexpr uint8 MemoryTraceVersion = 2;

////////////////////////////////////////////////////////////////////////////////
class FMallocWrapper
	: public FMalloc
{
public:
	FMallocWrapper(FMalloc* InMalloc);

private:
	struct FCookie
	{
		uint64 Tag  : 16;
		uint64 Bias : 8;
		uint64 Size : 40;
	};

	static uint32 GetActualAlignment(SIZE_T Size, uint32 Alignment);

	virtual void* Malloc(SIZE_T Size, uint32 Alignment) override;
	virtual void* Realloc(void* PrevAddress, SIZE_T NewSize, uint32 Alignment) override;
	virtual void Free(void* Address) override;
	virtual bool IsInternallyThreadSafe() const override                    { return InnerMalloc->IsInternallyThreadSafe(); }
	virtual void UpdateStats() override                                     { InnerMalloc->UpdateStats(); }
	virtual void GetAllocatorStats(FGenericMemoryStats& Out) override       { InnerMalloc->GetAllocatorStats(Out); }
	virtual void DumpAllocatorStats(FOutputDevice& Ar) override             { InnerMalloc->DumpAllocatorStats(Ar); }
	virtual bool ValidateHeap() override                                    { return InnerMalloc->ValidateHeap(); }
	virtual bool GetAllocationSize(void* Address, SIZE_T &SizeOut) override { return InnerMalloc->GetAllocationSize(Address, SizeOut); }
	virtual void SetupTLSCachesOnCurrentThread() override                   { return InnerMalloc->SetupTLSCachesOnCurrentThread(); }
	virtual void MarkTLSCachesAsUsedOnCurrentThread() override              { InnerMalloc->MarkTLSCachesAsUsedOnCurrentThread(); }
	virtual void MarkTLSCachesAsUnusedOnCurrentThread() override            { InnerMalloc->MarkTLSCachesAsUnusedOnCurrentThread(); }
	virtual void OnMallocInitialized() override                             { InnerMalloc->OnMallocInitialized(); }
	virtual void OnPreFork() override                                       { InnerMalloc->OnPreFork(); }
	virtual void OnPostFork() override                                      { InnerMalloc->OnPostFork(); }
	virtual const TCHAR* GetDescriptiveName() override                      { return InnerMalloc->GetDescriptiveName(); }

	FMalloc* InnerMalloc;
};

////////////////////////////////////////////////////////////////////////////////
FMallocWrapper::FMallocWrapper(FMalloc* InMalloc)
: InnerMalloc(InMalloc)
{
}

////////////////////////////////////////////////////////////////////////////////
uint32 FMallocWrapper::GetActualAlignment(SIZE_T Size, uint32 Alignment)
{
	// Defaults; if size is < 16 then alignment is 8 else 16.
	uint32 DefaultAlignment = 8 << uint32(Size >= 16);
	return (Alignment < DefaultAlignment) ? DefaultAlignment : Alignment;
}

////////////////////////////////////////////////////////////////////////////////
void* FMallocWrapper::Malloc(SIZE_T Size, uint32 Alignment)
{
	uint32 ActualAlignment = GetActualAlignment(Size, Alignment);
	void* Address = InnerMalloc->Malloc(Size, Alignment);

	MemoryTrace_Alloc((uint64)Address, Size, ActualAlignment);

	return Address;
}

////////////////////////////////////////////////////////////////////////////////
void* FMallocWrapper::Realloc(void* PrevAddress, SIZE_T NewSize, uint32 Alignment)
{
	// This simplifies things and means reallocs trace events are true reallocs
	if (PrevAddress == nullptr)
	{
		return Malloc(NewSize, Alignment);
	}

	MemoryTrace_ReallocFree((uint64)PrevAddress);

	void* RetAddress = InnerMalloc->Realloc(PrevAddress, NewSize, Alignment);

	Alignment = GetActualAlignment(NewSize, Alignment);
	MemoryTrace_ReallocAlloc((uint64)RetAddress, NewSize, Alignment);

	return RetAddress;
}

////////////////////////////////////////////////////////////////////////////////
void FMallocWrapper::Free(void* Address)
{
	if (Address == nullptr)
	{
		return;
	}

	MemoryTrace_Free((uint64)Address);

	void* InnerAddress = Address;

	return InnerMalloc->Free(InnerAddress);
}

////////////////////////////////////////////////////////////////////////////////
template <class T>
class alignas(alignof(T)) FUndestructed
{
public:
	template <typename... ArgTypes>
	void Construct(ArgTypes... Args)
	{
		::new (Buffer) T(Args...);
		bIsConstructed = true;
	}

	bool IsConstructed() const
	{
		return bIsConstructed;
	}

	T* operator & ()  { return (T*)Buffer; }
	T* operator -> () { return (T*)Buffer; }

protected:
	uint8 Buffer[sizeof(T)];
	bool bIsConstructed;
};

////////////////////////////////////////////////////////////////////////////////
static FUndestructed<FTraceMalloc> GTraceMalloc;

////////////////////////////////////////////////////////////////////////////////
template <typename ArgCharType>
static EMemoryTraceInit MemoryTrace_ShouldEnable(int32 ArgC, const ArgCharType* const* ArgV)
{
	EMemoryTraceInit Mode = EMemoryTraceInit::Disabled;

	for (int32 ArgIndex = 1; ArgIndex < ArgC; ArgIndex++)
	{
		const ArgCharType* Arg = ArgV[ArgIndex];
		TStringView<ArgCharType> ArgView(Arg);

		// In certain scenarios (for example when launching a packaged title on some platforms)
		// the multiple arguments are squashed into one single argument. Due to this we also need
		// to parse each argument as if it contains multiple sub arguments.
		int32 ArgStart = INDEX_NONE;
		while (ArgView.FindChar(CHARTEXT(ArgCharType, '-'), ArgStart))
		{
			ArgView.RightChopInline(ArgStart);

			int32 SpacePos = INDEX_NONE;
			ArgView.FindChar(CHARTEXT(ArgCharType, ' '), SpacePos);
			TStringView<ArgCharType> SubArg = (SpacePos == INDEX_NONE) ? ArgView : ArgView.Left(SpacePos);

			if (SubArg.Left(7).Equals(CHARTEXTVIEW(ArgCharType, "-trace="), ESearchCase::IgnoreCase))
			{
				TStringView<ArgCharType> ChannelsOrPresets = SubArg.RightChop(7);

				while (!ChannelsOrPresets.IsEmpty())
				{
					int32 CommaPos = INDEX_NONE;
					ChannelsOrPresets.FindChar(CHARTEXT(ArgCharType, ','), CommaPos);
					TStringView<ArgCharType> Channel = (CommaPos == INDEX_NONE) ? ChannelsOrPresets : ChannelsOrPresets.Left(CommaPos);

					if (Channel.Equals(CHARTEXTVIEW(ArgCharType, "memalloc"), ESearchCase::IgnoreCase))
					{
						Mode |= EMemoryTraceInit::AllocEvents;
						// The tags and memory scopes are also emitted on the "memalloc" channel.
						Mode |= EMemoryTraceInit::Tags;
					}
					else if (Channel.Equals(CHARTEXTVIEW(ArgCharType, "callstack"), ESearchCase::IgnoreCase))
					{
						Mode |= EMemoryTraceInit::Callstacks;
					}
					else if (Channel.Equals(CHARTEXTVIEW(ArgCharType, "memtag"), ESearchCase::IgnoreCase))
					{
						// The "memtag" channel is only used to enable LLM tracing.
					}
					else if (Channel.Equals(CHARTEXTVIEW(ArgCharType, "probing"), ESearchCase::IgnoreCase))
					{
						Mode |= EMemoryTraceInit::Probing;
					}
					else if (Channel.Equals(CHARTEXTVIEW(ArgCharType, "memory"), ESearchCase::IgnoreCase))
					{
						Mode |= EMemoryTraceInit::Full;
					}
					else if (Channel.Equals(CHARTEXTVIEW(ArgCharType, "memory_light"), ESearchCase::IgnoreCase))
					{
						Mode |= EMemoryTraceInit::Light;
					}

					ChannelsOrPresets.RightChopInline(Channel.Len() + 1);
				}
			}

			//Pass the '-' and find the next
			ArgView.RightChopInline(FMath::Max(SubArg.Len(),1));
		}
	}

	return Mode;
}

////////////////////////////////////////////////////////////////////////////////
FMalloc* MemoryTrace_CreateInternal(FMalloc* InMalloc, EMemoryTraceInit Mode)
{
	// If allocation events are not desired we don't need to do anything, even
	// if user has enabled only callstacks it will be enabled later.
	if (!EnumHasAnyFlags(Mode, EMemoryTraceInit::AllocEvents))
	{
		return InMalloc;
	}

	// Some OSes (i.e. Windows) will terminate all threads except the main
	// one as part of static deinit. However we may receive more memory
	// trace events that would get lost as Trace's worker thread has been
	// terminated. So flush the last remaining memory events trace needs
	// to be updated which we will do that in response to memory events.
	// We'll use an atexit can to know when Trace is probably no longer
	// getting ticked.
	atexit([]() { MemoryTrace_EnableTracePump(); });

	GTraceMalloc.Construct(InMalloc);

	// Both tag and callstack tracing need to use the wrapped trace malloc
	// so we can break out tracing memory overhead (and not cause recursive behaviour).
	if (EnumHasAnyFlags(Mode, EMemoryTraceInit::Tags))
	{
		MemoryTrace_InitTags(&GTraceMalloc);
	}
	if (EnumHasAnyFlags(Mode, EMemoryTraceInit::Callstacks))
	{
		CallstackTrace_Create(&GTraceMalloc);
	}
	if (EnumHasAnyFlags(Mode, EMemoryTraceInit::Probing))
	{
		MemoryTrace_InitMemoryProbe();
	}

	static FUndestructed<FMallocWrapper> SMallocWrapper;
	SMallocWrapper.Construct(InMalloc);

	return &SMallocWrapper;
}

////////////////////////////////////////////////////////////////////////////////
FMalloc* MemoryTrace_CreateInternal(FMalloc* InMalloc, int32 ArgC, const WIDECHAR* const* ArgV)
{
	const EMemoryTraceInit Mode = MemoryTrace_ShouldEnable(ArgC, ArgV);
	return MemoryTrace_CreateInternal(InMalloc, Mode);
}

////////////////////////////////////////////////////////////////////////////////
FMalloc* MemoryTrace_CreateInternal(FMalloc* InMalloc, int32 ArgC, const ANSICHAR* const* ArgV)
{
	const EMemoryTraceInit Mode = MemoryTrace_ShouldEnable(ArgC, ArgV);
	return MemoryTrace_CreateInternal(InMalloc, Mode);
}

////////////////////////////////////////////////////////////////////////////////
void MemoryTrace_Initialize()
{
	// At this point we initialized the system to allow tracing.
	GTraceAllowed = true;

	const FPlatformMemoryConstants& MemoryConstants = FPlatformMemory::GetConstants();

	UE_TRACE_LOG(Memory, Init, MemAllocChannel)
		<< Init.PageSize(MemoryConstants.PageSize)
		<< Init.MarkerPeriod(MarkerSamplePeriod + 1)
		<< Init.Version(MemoryTraceVersion)
		<< Init.MinAlignment(uint8(MIN_ALIGNMENT))
		<< Init.SizeShift(uint8(SizeShift));

	const HeapId SystemRootHeap = MemoryTrace_RootHeapSpec(TEXT("System memory"));
	check(SystemRootHeap == EMemoryTraceRootHeap::SystemMemory);
	const HeapId VideoRootHeap = MemoryTrace_RootHeapSpec(TEXT("Video memory"));
	check(VideoRootHeap == EMemoryTraceRootHeap::VideoMemory);

	static_assert((1 << SizeShift) - 1 <= MIN_ALIGNMENT, "Not enough bits to pack size fields");

#if !UE_MEMORY_TRACE_LATE_INIT
	// On some platforms callstack initialization cannot happen this early in the process. It is initialized
	// in other locations when UE_MEMORY_TRACE_LATE_INIT is defined. Until that point allocations cannot have
	// callstacks.
	CallstackTrace_Initialize();
#endif
}

////////////////////////////////////////////////////////////////////////////////
bool MemoryTrace_IsActive()
{
	return GTraceAllowed;
}

////////////////////////////////////////////////////////////////////////////////
void MemoryTrace_EnableTracePump()
{
	GDoPumpTrace = true;
}

////////////////////////////////////////////////////////////////////////////////
void MemoryTrace_UpdateInternal()
{
	const uint32 TheCount = GMarkerCounter.fetch_add(1, std::memory_order_relaxed);
	if ((TheCount & MarkerSamplePeriod) == 0)
	{
		UE_TRACE_LOG(Memory, Marker, MemAllocChannel)
			<< Marker.Cycle(FPlatformTime::Cycles64());
	}

	if (GDoPumpTrace)
	{
		UE::Trace::Update();
	}
}

////////////////////////////////////////////////////////////////////////////////
void MemoryTrace_Alloc(uint64 Address, uint64 Size, uint32 Alignment, HeapId RootHeap, uint32 ExternalCallstackId)
{
	if (!GTraceAllowed)
	{
		return;
	}

	checkSlow(RootHeap < 16);

	const uint32 AlignmentPow2 = uint32(FPlatformMath::CountTrailingZeros(Alignment));
	const uint32 Alignment_SizeLower = (AlignmentPow2 << SizeShift) | uint32(Size & ((1 << SizeShift) - 1));
	const uint32 CallstackId = ExternalCallstackId ? ExternalCallstackId : GDoNotAllocateInTrace ? 0 : CallstackTrace_GetCurrentId();

	switch (RootHeap)
	{
		case EMemoryTraceRootHeap::SystemMemory:
		{
			UE_TRACE_LOG(Memory, AllocSystem, MemAllocChannel)
				<< AllocSystem.Address(uint64(Address))
				<< AllocSystem.CallstackId(CallstackId)
				<< AllocSystem.Size(uint32(Size >> SizeShift))
				<< AllocSystem.AlignmentPow2_SizeLower(uint8(Alignment_SizeLower));
			break;
		}

		case EMemoryTraceRootHeap::VideoMemory:
		{
			UE_TRACE_LOG(Memory, AllocVideo, MemAllocChannel)
				<< AllocVideo.Address(uint64(Address))
				<< AllocVideo.CallstackId(CallstackId)
				<< AllocVideo.Size(uint32(Size >> SizeShift))
				<< AllocVideo.AlignmentPow2_SizeLower(uint8(Alignment_SizeLower));
			break;
		}

		default:
		{
			UE_TRACE_LOG(Memory, Alloc, MemAllocChannel)
				<< Alloc.Address(uint64(Address))
				<< Alloc.CallstackId(CallstackId)
				<< Alloc.Size(uint32(Size >> SizeShift))
				<< Alloc.AlignmentPow2_SizeLower(uint8(Alignment_SizeLower))
				<< Alloc.RootHeap(uint8(RootHeap));
			break;
		}
	}

	MemoryTrace_UpdateInternal();
}

////////////////////////////////////////////////////////////////////////////////
void MemoryTrace_Free(uint64 Address, HeapId RootHeap, uint32 ExternalCallstackId)
{
	if (!GTraceAllowed)
	{
		return;
	}

	checkSlow(RootHeap < 16);

	const uint32 CallstackId = ExternalCallstackId ? ExternalCallstackId : GDoNotAllocateInTrace ? 0 : CallstackTrace_GetCurrentId();

	switch (RootHeap)
	{
		case EMemoryTraceRootHeap::SystemMemory:
			{
				UE_TRACE_LOG(Memory, FreeSystem, MemAllocChannel)
					<< FreeSystem.Address(uint64(Address))
					<< FreeSystem.CallstackId(CallstackId);
				break;
			}
		case EMemoryTraceRootHeap::VideoMemory:
			{
				UE_TRACE_LOG(Memory, FreeVideo, MemAllocChannel)
					<< FreeVideo.Address(uint64(Address))
					<< FreeVideo.CallstackId(CallstackId);
				break;
			}
		default:
			{
				UE_TRACE_LOG(Memory, Free, MemAllocChannel)
					<< Free.Address(uint64(Address))
					<< Free.CallstackId(CallstackId)
					<< Free.RootHeap(uint8(RootHeap));
				break;
			}
	}

	MemoryTrace_UpdateInternal();
}

////////////////////////////////////////////////////////////////////////////////
void MemoryTrace_ReallocAlloc(uint64 Address, uint64 Size, uint32 Alignment, HeapId RootHeap, uint32 ExternalCallstackId)
{
	if (!GTraceAllowed)
	{
		return;
	}

	checkSlow(RootHeap < 16);

	const uint32 AlignmentPow2 = uint32(FPlatformMath::CountTrailingZeros(Alignment));
	const uint32 Alignment_SizeLower = (AlignmentPow2 << SizeShift) | uint32(Size & ((1 << SizeShift) - 1));
	const uint32 CallstackId = ExternalCallstackId ? ExternalCallstackId : GDoNotAllocateInTrace ? 0 : CallstackTrace_GetCurrentId();

	switch (RootHeap)
	{
		case EMemoryTraceRootHeap::SystemMemory:
		{
			UE_TRACE_LOG(Memory, ReallocAllocSystem, MemAllocChannel)
				<< ReallocAllocSystem.Address(uint64(Address))
				<< ReallocAllocSystem.CallstackId(CallstackId)
				<< ReallocAllocSystem.Size(uint32(Size >> SizeShift))
				<< ReallocAllocSystem.AlignmentPow2_SizeLower(uint8(Alignment_SizeLower));
			break;
		}

		default:
		{
			UE_TRACE_LOG(Memory, ReallocAlloc, MemAllocChannel)
				<< ReallocAlloc.Address(uint64(Address))
				<< ReallocAlloc.CallstackId(CallstackId)
				<< ReallocAlloc.Size(uint32(Size >> SizeShift))
				<< ReallocAlloc.AlignmentPow2_SizeLower(uint8(Alignment_SizeLower))
				<< ReallocAlloc.RootHeap(uint8(RootHeap));
			break;
		}
	}

	MemoryTrace_UpdateInternal();
}

////////////////////////////////////////////////////////////////////////////////
void MemoryTrace_ReallocFree(uint64 Address, HeapId RootHeap, uint32 ExternalCallstackId)
{
	if (!GTraceAllowed)
	{
		return;
	}

	checkSlow(RootHeap < 16);

	const uint32 CallstackId = ExternalCallstackId ? ExternalCallstackId : GDoNotAllocateInTrace ? 0 : CallstackTrace_GetCurrentId();

	switch (RootHeap)
	{
		case EMemoryTraceRootHeap::SystemMemory:
		{
			UE_TRACE_LOG(Memory, ReallocFreeSystem, MemAllocChannel)
				<< ReallocFreeSystem.Address(uint64(Address))
				<< ReallocFreeSystem.CallstackId(CallstackId);
			break;
		}

		default:
		{
			UE_TRACE_LOG(Memory, ReallocFree, MemAllocChannel)
				<< ReallocFree.Address(uint64(Address))
				<< ReallocFree.CallstackId(CallstackId)
				<< ReallocFree.RootHeap(uint8(RootHeap));
			break;
		}
	}

	MemoryTrace_UpdateInternal();
}

////////////////////////////////////////////////////////////////////////////////
void MemoryTrace_UpdateAlloc(uint64 Address, HeapId RootHeap, uint32 ExternalCallstackId)
{
	if (!GTraceAllowed)
	{
		return;
	}

	checkSlow(RootHeap < 16);

	const uint32 CallstackId = ExternalCallstackId ? ExternalCallstackId : GDoNotAllocateInTrace ? 0 : CallstackTrace_GetCurrentId();

	UE_TRACE_LOG(Memory, UpdateAlloc, MemAllocChannel)
		<< UpdateAlloc.Address(uint64(Address))
		<< UpdateAlloc.CallstackId(CallstackId)
		<< UpdateAlloc.RootHeap(uint8(RootHeap));

	MemoryTrace_UpdateInternal();
}

////////////////////////////////////////////////////////////////////////////////
void MemoryTrace_SwapOp(uint64 PageAddress, EMemoryTraceSwapOperation SwapOperation, uint32 CompressedSize, uint32 CallstackId)
{
	if (!GTraceAllowed)
	{
		return;
	}

	UE_TRACE_LOG(Memory, MemorySwapOp, MemAllocChannel)
		<< MemorySwapOp.Address(PageAddress)
		<< MemorySwapOp.CallstackId(CallstackId)
		<< MemorySwapOp.CompressedSize(CompressedSize)
		<< MemorySwapOp.SwapOp((uint8)SwapOperation);

	MemoryTrace_UpdateInternal();
}

////////////////////////////////////////////////////////////////////////////////
HeapId MemoryTrace_HeapSpec(HeapId ParentId, const TCHAR* Name, EMemoryTraceHeapFlags Flags)
{
	if (!GTraceAllowed)
	{
		return 0;
	}

	static std::atomic<HeapId> HeapIdCount(EMemoryTraceRootHeap::EndReserved + 1); //Reserve indexes for root heaps
	const HeapId Id = HeapIdCount.fetch_add(1);
	const uint32 NameLen = FCString::Strlen(Name);
	const uint32 DataSize = NameLen * sizeof(TCHAR);
	check(ParentId < Id);

	UE_TRACE_LOG(Memory, HeapSpec, MemAllocChannel, DataSize)
		<< HeapSpec.Id(Id)
		<< HeapSpec.ParentId(ParentId)
		<< HeapSpec.Name(Name, NameLen)
		<< HeapSpec.Flags(uint16(Flags));

	return Id;
}

////////////////////////////////////////////////////////////////////////////////
HeapId MemoryTrace_RootHeapSpec(const TCHAR* Name, EMemoryTraceHeapFlags Flags)
{
	if (!GTraceAllowed)
	{
		return 0;
	}

	static std::atomic<HeapId> RootHeapCount(0);
	const HeapId Id = RootHeapCount.fetch_add(1);
	check(Id <= EMemoryTraceRootHeap::EndReserved);

	const uint32 NameLen = FCString::Strlen(Name);
	const uint32 DataSize = NameLen * sizeof(TCHAR);

	UE_TRACE_LOG(Memory, HeapSpec, MemAllocChannel, DataSize)
		<< HeapSpec.Id(Id)
		<< HeapSpec.ParentId(HeapId(~0))
		<< HeapSpec.Name(Name, NameLen)
		<< HeapSpec.Flags(uint16(EMemoryTraceHeapFlags::Root|Flags));

	return Id;
}

////////////////////////////////////////////////////////////////////////////////
void MemoryTrace_MarkAllocAsHeap(uint64 Address, HeapId Heap, EMemoryTraceHeapAllocationFlags Flags, uint32 ExternalCallstackId)
{
	if (!GTraceAllowed)
	{
		return;
	}

	const uint32 CallstackId = ExternalCallstackId ? ExternalCallstackId : GDoNotAllocateInTrace ? 0 : CallstackTrace_GetCurrentId();

	UE_TRACE_LOG(Memory, HeapMarkAlloc, MemAllocChannel)
		<< HeapMarkAlloc.Address(uint64(Address))
		<< HeapMarkAlloc.CallstackId(CallstackId)
		<< HeapMarkAlloc.Flags(uint16(EMemoryTraceHeapAllocationFlags::Heap | Flags))
		<< HeapMarkAlloc.Heap(Heap);
}

////////////////////////////////////////////////////////////////////////////////
void MemoryTrace_UnmarkAllocAsHeap(uint64 Address, HeapId Heap, uint32 ExternalCallstackId)
{
	if (!GTraceAllowed)
	{
		return;
	}

	const uint32 CallstackId = ExternalCallstackId ? ExternalCallstackId : GDoNotAllocateInTrace ? 0 : CallstackTrace_GetCurrentId();

	// Sets all flags to zero
	UE_TRACE_LOG(Memory, HeapUnmarkAlloc, MemAllocChannel)
		<< HeapUnmarkAlloc.Address(uint64(Address))
		<< HeapUnmarkAlloc.CallstackId(CallstackId)
		<< HeapUnmarkAlloc.Heap(Heap);
}

/////////////////////////////////////////////////////////////////////////////
FMalloc* MemoryTrace_GetAllocator()
{
	if (GTraceMalloc.IsConstructed())
	{
		return &GTraceMalloc;
	}
	return nullptr;
}


#else // UE_MEMORY_TRACE_ENABLED

/////////////////////////////////////////////////////////////////////////////
bool MemoryTrace_IsActive()
{
	return false;
}

#endif // UE_MEMORY_TRACE_ENABLED

/////////////////////////////////////////////////////////////////////////////
FTraceMalloc::FTraceMalloc(FMalloc* InMalloc)
{
	WrappedMalloc = InMalloc;
}

/////////////////////////////////////////////////////////////////////////////
FTraceMalloc::~FTraceMalloc()
{
}

/////////////////////////////////////////////////////////////////////////////
void* FTraceMalloc::Malloc(SIZE_T Count, uint32 Alignment)
{
#if UE_MEMORY_TRACE_ENABLED
	UE_TRACE_METADATA_CLEAR_SCOPE();
	UE_MEMSCOPE(TRACE_TAG);

	void* NewPtr;
	{
		TGuardValue<bool> _(GDoNotAllocateInTrace, true);
		NewPtr = WrappedMalloc->Malloc(Count, Alignment);
	}

	const uint64 Size = Count;
	const uint32 AlignmentPow2 = uint32(FPlatformMath::CountTrailingZeros(Alignment));
	const uint32 Alignment_SizeLower = (AlignmentPow2 << SizeShift) | uint32(Size & ((1 << SizeShift) - 1));

	UE_TRACE_LOG(Memory, Alloc, MemAllocChannel)
		<< Alloc.Address(uint64(NewPtr))
		<< Alloc.CallstackId(0)
		<< Alloc.Size(uint32(Size >> SizeShift))
		<< Alloc.AlignmentPow2_SizeLower(uint8(Alignment_SizeLower))
		<< Alloc.RootHeap(uint8(EMemoryTraceRootHeap::SystemMemory));

	return NewPtr;
#else
	return WrappedMalloc->Malloc(Count, Alignment);
#endif //UE_MEMORY_TRACE_ENABLED
}

/////////////////////////////////////////////////////////////////////////////
void* FTraceMalloc::Realloc(void* Original, SIZE_T Count, uint32 Alignment)
{
#if UE_MEMORY_TRACE_ENABLED
	UE_TRACE_METADATA_CLEAR_SCOPE();
	UE_MEMSCOPE(TRACE_TAG);

	UE_TRACE_LOG(Memory, ReallocFree, MemAllocChannel)
		<< ReallocFree.Address(uint64(Original))
		<< ReallocFree.RootHeap(uint8(EMemoryTraceRootHeap::SystemMemory));

	void* NewPtr;
	{
		TGuardValue<bool> _(GDoNotAllocateInTrace, true);
		NewPtr = WrappedMalloc->Realloc(Original, Count, Alignment);
	}

	const uint64 Size = Count;
	const uint32 AlignmentPow2 = uint32(FPlatformMath::CountTrailingZeros(Alignment));
	const uint32 Alignment_SizeLower = (AlignmentPow2 << SizeShift) | uint32(Size & ((1 << SizeShift) - 1));

	UE_TRACE_LOG(Memory, ReallocAlloc, MemAllocChannel)
		<< ReallocAlloc.Address(uint64(NewPtr))
		<< ReallocAlloc.CallstackId(0)
		<< ReallocAlloc.Size(uint32(Size >> SizeShift))
		<< ReallocAlloc.AlignmentPow2_SizeLower(uint8(Alignment_SizeLower))
		<< ReallocAlloc.RootHeap(uint8(EMemoryTraceRootHeap::SystemMemory));

	return NewPtr;
#else
	return WrappedMalloc->Realloc(Original, Count, Alignment);
#endif //UE_MEMORY_TRACE_ENABLED
}

/////////////////////////////////////////////////////////////////////////////
void FTraceMalloc::Free(void* Original)
{
#if UE_MEMORY_TRACE_ENABLED
	UE_TRACE_LOG(Memory, Free, MemAllocChannel)
		<< Free.Address(uint64(Original))
		<< Free.RootHeap(uint8(EMemoryTraceRootHeap::SystemMemory));

	{
		TGuardValue<bool> _(GDoNotAllocateInTrace, true);
		WrappedMalloc->Free(Original);
	}
#else
	WrappedMalloc->Free(Original);
#endif //UE_MEMORY_TRACE_ENABLED
}
