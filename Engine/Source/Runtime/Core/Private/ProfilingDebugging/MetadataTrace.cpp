// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProfilingDebugging/MetadataTrace.h"

#if UE_TRACE_METADATA_ENABLED
////////////////////////////////////////////////////////////////////////////////////////////////////

UE_TRACE_CHANNEL_DEFINE(MetadataChannel, "Metadata stack management for trace context propagation. Associated with memory allocations.");

UE_TRACE_EVENT_DEFINE(MetadataStack, ClearScope);
UE_TRACE_EVENT_DEFINE(MetadataStack, SaveStack);
UE_TRACE_EVENT_DEFINE(MetadataStack, RestoreStack);

////////////////////////////////////////////////////////////////////////////////////////////////////
UE_AUTORTFM_ALWAYS_OPEN
static uint32 AtomicMetadataCounter()
{
	static std::atomic_uint32_t Counter(1);
	return Counter.fetch_add(1u);
}

////////////////////////////////////////////////////////////////////////////////////////////////////
uint32 FMetadataTrace::SaveStack()
{
	const uint32 Id = AtomicMetadataCounter();
	UE_TRACE_LOG(MetadataStack, SaveStack, MetadataChannel)
			<< SaveStack.Id(Id);
	return Id;
}

////////////////////////////////////////////////////////////////////////////////////////////////////
FMetadataRestoreScope::FMetadataRestoreScope(uint32 SavedMetadataIdentifier)
{
	if (UE_TRACE_CHANNELEXPR_IS_ENABLED(MetadataChannel) & (SavedMetadataIdentifier != 0))
	{
		ActivateScope(SavedMetadataIdentifier);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
FMetadataRestoreScope::~FMetadataRestoreScope()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////
void FMetadataRestoreScope::ActivateScope(uint32 InStackId)
{
	if (auto LogScope = FMetadataStackRestoreStackFields::LogScopeType::ScopedEnter<FMetadataStackRestoreStackFields>())
	{
		if (const auto& __restrict MemoryScope = *(FMetadataStackRestoreStackFields*)(&LogScope))
		{
			Inner.SetActive();
			LogScope += LogScope << MemoryScope.Id(InStackId);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
#endif // UE_TRACE_METADATA_ENABLED
