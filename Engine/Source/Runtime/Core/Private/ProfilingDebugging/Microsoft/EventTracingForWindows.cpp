// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProfilingDebugging/PlatformEvents.h"
#include "HAL/Runnable.h"
#include "HAL/Event.h"
#include "HAL/RunnableThread.h"
#include "HAL/PlatformProcess.h"
#include "Logging/LogMacros.h"
#include "Experimental/Containers/SherwoodHashTable.h"
#include "Trace/Trace.h"

#if PLATFORM_SUPPORTS_PLATFORM_EVENTS

#include COMPILED_PLATFORM_HEADER(PlatformEventTracingForWindows.h) // IWYU pragma: export

/////////////////////////////////////////////////////////////////////

DEFINE_LOG_CATEGORY_STATIC(LogPlatformEvents, Log, All);

/////////////////////////////////////////////////////////////////////

namespace {

struct FHandleScope
{
	FHandleScope(HANDLE InHandle)
		: Handle(InHandle)
	{}

	FHandleScope()
		: Handle(INVALID_HANDLE_VALUE)
	{}

	~FHandleScope()
	{ 
		CloseHandle(Handle); 
	}

	operator HANDLE() const { return Handle; }

	HANDLE Handle;
};

class FPlatformEvents : public FRunnable
{
public:
	FPlatformEvents(uint32 SamplingIntervalUsec);
	~FPlatformEvents();

	bool CanEnable(FPlatformEventsTrace::EEventType Event, const TCHAR** OutReason);
	void Enable(FPlatformEventsTrace::EEventType Event);
	void Disable(FPlatformEventsTrace::EEventType Event);

private:
	virtual uint32 Run() override;
	virtual void Stop() override;

	bool StartETW();
	void StopETW();

	static void WINAPI TraceEventCallback(PEVENT_RECORD Event);
	static uint32 CurrentProcessId;

	bool bStarting = false;
	bool bError = false;
	FEventRef ReadyEvent;
	FRunnableThread* Thread = nullptr;
	uint32 SamplingIntervalUsec;

	FPlatformEventsTrace::EEventType EnabledEvents = FPlatformEventsTrace::EEventType::None;

	struct FTrace
	{
		EVENT_TRACE_PROPERTIES Properties;
		WCHAR SessionName[1024];
	};

	TRACEHANDLE TraceHandle = INVALID_PROCESSTRACE_HANDLE;
	FTrace Trace;

	// timestamp for each core when thread start executing on it
	uint64 ContexSwitchStart[256];

	// thread ids that have their info already sent
	Experimental::TSherwoodSet<uint32> ThreadNameSet;
};

// https://docs.microsoft.com/en-us/windows/win32/etw/cswitch
struct FContextSwitchEvent
{
	enum { Type = 36 };

	uint32 NewThreadId;					// 0
	uint32 OldThreadId;					// 4
	uint8  NewThreadPriority;			// 8
	uint8  OldThreadPriority;			// 9
	uint8  PreviousCState;				// 10
	uint8  SpareByte;					// 11
	uint8  OldThreadWaitReason;			// 12
	uint8  OldThreadWaitMode;			// 13
	uint8  OldThreadState;				// 14
	uint8  OldThreadWaitIdealProcessor;	// 15
	uint32 NewThreadWaitTime;			// 16
	uint32 Reserved;					// 20
	uint32 Reserved2;					// 24 (undocumented)
};

// https://docs.microsoft.com/en-us/windows/win32/etw/stackwalk-event
struct FStackWalkEvent
{
	enum { Type = 32 };

	uint64 TimeStamp;
	uint32 ProcessId;
	uint32 ThreadId;

	// this array can be max 192 elements
	// actual count is determined by total byte size of event
	uint64 Stack[];
};

} // namespace

/////////////////////////////////////////////////////////////////////

static FPlatformEvents* GPlatformEvents;

/////////////////////////////////////////////////////////////////////

uint32 FPlatformEvents::CurrentProcessId;

void FPlatformEvents::TraceEventCallback(PEVENT_RECORD Event)
{
	int Opcode = Event->EventHeader.EventDescriptor.Opcode;
	if (Opcode == FContextSwitchEvent::Type)
	{
		const FContextSwitchEvent* Payload = (const FContextSwitchEvent*)Event->UserData;
		
		// Recent updates of ETW changes the size of the payload somewhat.
		constexpr uint32 UserDataSizeOld = sizeof(FContextSwitchEvent) - 4;
		constexpr uint32 UserDataSizeNew = sizeof(FContextSwitchEvent);
		if (Payload == nullptr ||
			(Event->UserDataLength != UserDataSizeOld &&
			 Event->UserDataLength != UserDataSizeNew))
		{
			return;
		}

		uint8 CoreNumber = Event->BufferContext.ProcessorNumber;

		uint64 StartTime = GPlatformEvents->ContexSwitchStart[CoreNumber];
		uint64 EndTime = Event->EventHeader.TimeStamp.QuadPart;

		if (StartTime != 0)
		{
			uint32 ThreadId = Payload->OldThreadId;

			if (ThreadId != 0)
			{
				FPlatformEventsTrace::OutputContextSwitch(StartTime, EndTime, ThreadId, CoreNumber);

				bool bAlreadyAdded = false;
				GPlatformEvents->ThreadNameSet.Add(ThreadId, &bAlreadyAdded);
				if (!bAlreadyAdded)
				{
					WCHAR Name[4096];
					DWORD NameLen = 0;
					uint32 ProcessId = 0;

					HANDLE ThreadHandle = ::OpenThread(THREAD_QUERY_LIMITED_INFORMATION, false, ThreadId);
					if (ThreadHandle)
					{
						ProcessId = ::GetProcessIdOfThread(ThreadHandle);
						::CloseHandle(ThreadHandle);
					}

					// collect names only about threads not belonging to our process
					if (ProcessId && ProcessId != CurrentProcessId)
					{
						HANDLE ProcessHandle = ::OpenProcess(PROCESS_QUERY_LIMITED_INFORMATION, false, ProcessId);
						if (ProcessHandle)
						{
							NameLen = sizeof(Name) / sizeof(*Name);
							if (::QueryFullProcessImageNameW(ProcessHandle, 0, Name, &NameLen) == 0)
							{
								// failed to get name of process executable
								NameLen = 0;
							}
							::CloseHandle(ProcessHandle);
						}

						FPlatformEventsTrace::OutputThreadName(ThreadId, ProcessId, Name, NameLen);
					}
				}
			}
		}

		GPlatformEvents->ContexSwitchStart[CoreNumber] = EndTime;
	}
	else if (Opcode == FStackWalkEvent::Type)
	{
		const FStackWalkEvent* Payload = (const FStackWalkEvent*)Event->UserData;
		if (Payload == nullptr ||
			Payload->ProcessId != CurrentProcessId ||
			Event->UserDataLength < sizeof(FStackWalkEvent))
		{
			return;
		}

		uint32 Count = (Event->UserDataLength - sizeof(FStackWalkEvent)) / sizeof(uint64);

		// Sometimes we get call stack with only one or two entries, which obviously is not very useful.
		//if (Count <= 2)
		//{
		//	return;
		//}

		// Sometimes stack walk from ETW gives a call-stack that is fully in kernel address space.
		// So, if all addresses have a high bit set, then ignore this call stack.
		bool bValid = false;
		for (uint32 Index = 0; Index < Count; Index++)
		{
			uint64 Address = Payload->Stack[Index];
			if ((Address >> 63) == 0)
			{
				bValid = true;
				break;
			}
		}
		if (bValid)
		{
			FPlatformEventsTrace::OutputStackSample(Payload->TimeStamp, Payload->ThreadId, Payload->Stack, Count);
		}
	}
}

/////////////////////////////////////////////////////////////////////

FPlatformEvents::FPlatformEvents(uint32 SamplingIntervalUsec)
	: ReadyEvent(EEventMode::ManualReset)
	, SamplingIntervalUsec(SamplingIntervalUsec)
{
	ZeroMemory(ContexSwitchStart, sizeof(ContexSwitchStart));
	CurrentProcessId = ::GetCurrentProcessId();

	ReadyEvent->Trigger();
}

/////////////////////////////////////////////////////////////////////

FPlatformEvents::~FPlatformEvents()
{
	Stop();
}

/////////////////////////////////////////////////////////////////////
bool FPlatformEvents::CanEnable(FPlatformEventsTrace::EEventType Event, const TCHAR** OutReason)
{
	// In order to control tracing the process must either run under
	// elevated privileges OR the user has to be member of PERF_LOG_USERS
	static const TCHAR* ErrProcAccess = TEXT("Cannot access process.");
	static const TCHAR* ErrProcRights = TEXT("Process does not have privileges to start event tracing.");
	
#if PLATFORM_WINDOWS
	FHandleScope CurrentProcToken;
	if (!::OpenProcessToken(GetCurrentProcess(), TOKEN_QUERY, &CurrentProcToken.Handle))
	{
		if (OutReason)
		{
			*OutReason = ErrProcAccess;
		}
		UE_LOGF(LogCore, Log, "Unable to enable platform events: %ls", ErrProcAccess);
		return false;
	}

	// Get the elevation token
	TOKEN_ELEVATION ElevationToken;
	DWORD ReturnSize = 0;
	if (!::GetTokenInformation(CurrentProcToken, TokenElevation, &ElevationToken, sizeof(TOKEN_ELEVATION), &ReturnSize))
	{
		if (OutReason)
		{
			*OutReason = ErrProcAccess;
		}
		UE_LOGF(LogCore, Log, "Unable to enable platform events: %ls", ErrProcAccess);
		return false;
	}
	
	// Check if the user is part of PERF_LOG_USERS
	// todo: Implement

	if (ElevationToken.TokenIsElevated /* || in perf log users */)
	{
		return true;
	}
	
	if (OutReason)
	{
		*OutReason = ErrProcRights;
	}
	
	UE_LOGF(LogCore, Log, "Unable to enable platform events: %ls", ErrProcRights);
	return false;
#else
	return true;
#endif
}

/////////////////////////////////////////////////////////////////////

void FPlatformEvents::Enable(FPlatformEventsTrace::EEventType Event)
{
	if (bStarting)
	{
		// in case ETW is already starting, wait for startup code to complete
		ReadyEvent->Wait(MAX_uint32);
	}

	if (bError)
	{
		// in case ETW failed to start up earlier, do nothing
		return;
	}

	if ((EnabledEvents & Event) != FPlatformEventsTrace::EEventType::None)
	{
		// if event is already enabled, do nothing
		return;
	}

	if (TraceHandle == INVALID_PROCESSTRACE_HANDLE)
	{
		// in case ETW is not started, start it now
		ReadyEvent->Reset();
		bError = false;
		bStarting = true;
		Thread = FRunnableThread::Create(this, TEXT("PlatformEvents"), 0, TPri_Normal);
	}
	else
	{
		// in case ETW was already running, enable the necessary event

		EVENT_TRACE_PROPERTIES* Properties = &Trace.Properties;
		if (Event == FPlatformEventsTrace::EEventType::ContextSwitch)
		{
			Properties->EnableFlags |= EVENT_TRACE_FLAG_CSWITCH;
		}
		else if (Event == FPlatformEventsTrace::EEventType::StackSampling)
		{
			Properties->EnableFlags |= EVENT_TRACE_FLAG_PROFILE;
		}

		ULONG Status = ::ControlTraceW(NULL, KERNEL_LOGGER_NAMEW, Properties, EVENT_TRACE_CONTROL_UPDATE);
		if (Status != ERROR_SUCCESS)
		{
			UE_LOGF(LogPlatformEvents, Error, "Unable to update ETW trace: 0x%08x", Status);
		}
	}

	EnumAddFlags(EnabledEvents, Event);
}

/////////////////////////////////////////////////////////////////////

void FPlatformEvents::Disable(FPlatformEventsTrace::EEventType Event)
{
	if (bStarting)
	{
		// in case ETW is already starting, wait for startup code to complete
		ReadyEvent->Wait(MAX_uint32);
	}

	if (TraceHandle == INVALID_PROCESSTRACE_HANDLE)
	{
		// in case ETW is not running, do nothing
		return;
	}

	if ((EnabledEvents & Event) == FPlatformEventsTrace::EEventType::None)
	{
		// if event is already disabled, do nothing
		return;
	}

	EnumRemoveFlags(EnabledEvents, Event);

	if (EnabledEvents == FPlatformEventsTrace::EEventType::None)
	{
		// in case all events are disabled, stop ETW
		Stop();
	}
	else
	{
		// in case ETW still needs to run, disable only specific event

		EVENT_TRACE_PROPERTIES* Properties = &Trace.Properties;
		if (Event == FPlatformEventsTrace::EEventType::ContextSwitch)
		{
			Properties->EnableFlags &= ~EVENT_TRACE_FLAG_CSWITCH;
		}
		else if (Event == FPlatformEventsTrace::EEventType::StackSampling)
		{
			Properties->EnableFlags &= ~EVENT_TRACE_FLAG_PROFILE;
		}

		ReadyEvent->Wait(MAX_uint32);
		ULONG Status = ::ControlTraceW(NULL, KERNEL_LOGGER_NAMEW, Properties, EVENT_TRACE_CONTROL_UPDATE);
		if (Status != ERROR_SUCCESS)
		{
			UE_LOGF(LogPlatformEvents, Error, "Unable to update ETW trace: 0x%08x", Status);
		}
	}
}

/////////////////////////////////////////////////////////////////////

uint32 FPlatformEvents::Run()
{
	if (StartETW())
	{
		ReadyEvent->Trigger();
		bStarting = false;

		// ProcessTrace is blocking function that enters processing loop
		// and calls TraceEventCallback callback for each event
		// it exits processing loop when trace is closed with ::CloseTrace
		::ProcessTrace(&TraceHandle, 1, NULL, NULL);
	}
	else
	{
		// in case of error in StartETW we want to be sure ETW is stopped
		StopETW();
		bError = true;
		bStarting = false;
		ReadyEvent->Trigger();
	}
	return 0;
}

/////////////////////////////////////////////////////////////////////

void FPlatformEvents::Stop()
{
	// because ETW is started in background thread, then first make sure
	// ETW startup code has completed (successfully or not) before actually
	// stopping it
	ReadyEvent->Wait(MAX_uint32);

	if (Thread)
	{
		StopETW();
		Thread->WaitForCompletion();
	}

	ThreadNameSet.Empty();
}

/////////////////////////////////////////////////////////////////////

bool FPlatformEvents::StartETW()
{
	// explicitly define GUIDs to simplify dependency on special defines in windows headers
	// https://docs.microsoft.com/en-us/windows/win32/etw/nt-kernel-logger-constants
	static const GUID ETW_PerfInfoGuid = { 0xce1dbfb4, 0x137e, 0x4da6, { 0x87, 0xb0, 0x3f, 0x59, 0xaa, 0x10, 0x2c, 0xbc } };
	static const GUID ETW_SystemTraceControlGuid = { 0x9e814aad, 0x3204, 0x11d2, { 0x9a, 0x82, 0x00, 0x60, 0x08, 0xa8, 0x69, 0x39 } };

	TRACEHANDLE Session;

	SYSTEM_INFO SysInfo;
	GetSystemInfo(&SysInfo);

	EVENT_TRACE_PROPERTIES* Properties = &Trace.Properties;

	// stop existing trace in case it is already running (from other tools)
	{
		ZeroMemory(Properties, sizeof(*Properties));
		Properties->Wnode.BufferSize = sizeof(Trace);
		Properties->Wnode.Guid = ETW_SystemTraceControlGuid;
		Properties->LogFileMode = EVENT_TRACE_REAL_TIME_MODE;
		Properties->LoggerNameOffset = sizeof(Trace.Properties);
		::ControlTraceW(NULL, KERNEL_LOGGER_NAMEW, Properties, EVENT_TRACE_CONTROL_STOP);
	}

	// enable profiling privilege for process
	{
		HANDLE Token;
		if (::OpenProcessToken(::GetCurrentProcess(), TOKEN_ADJUST_PRIVILEGES, &Token))
		{
			LUID Luid;
			if (::LookupPrivilegeValue(NULL, SE_SYSTEM_PROFILE_NAME, &Luid))
			{
				TOKEN_PRIVILEGES TokenPrivileges;
				TokenPrivileges.PrivilegeCount = 1;
				TokenPrivileges.Privileges[0].Luid = Luid;
				TokenPrivileges.Privileges[0].Attributes = SE_PRIVILEGE_ENABLED;
				if (!::AdjustTokenPrivileges(Token, 0, &TokenPrivileges, sizeof(TokenPrivileges), NULL, NULL))
				{
					uint32 Error = ::GetLastError();
					UE_LOGF(LogPlatformEvents, Warning, "Cannot enable profile privilege for process: 0x%08x", Error);
				}
			}
			::CloseHandle(Token);
		}
	}

	// set up sampling interval, must be done before StartTrace
	{
		typedef struct
		{
			ULONG Source;
			ULONG Interval;
		}
		MOCK_TRACE_PROFILE_INTERVAL;

		// interval is specified in 100 nanosecond units, and allowed range is limited
		// see "-setprofint" argument in https://docs.microsoft.com/en-us/windows-hardware/test/wpt/start
		MOCK_TRACE_PROFILE_INTERVAL Interval;
		Interval.Source = 0;
		Interval.Interval = FMath::Clamp<ULONG>(10 * SamplingIntervalUsec, 1221, 10000000);

		ULONG Status = ::TraceSetInformation(NULL, TraceSampledProfileIntervalInfo, &Interval, sizeof(Interval));
		if (Status != ERROR_SUCCESS)
		{
			UE_LOGF(LogPlatformEvents, Warning, "Unable to set ETW sampling interval: 0x%08x", Status);
		}
	}

	// setup trace properties
	ZeroMemory(Properties, sizeof(*Properties));
	Properties->Wnode.BufferSize = sizeof(Trace);
	Properties->Wnode.Guid = ETW_SystemTraceControlGuid;

	// NOTE: change this depending on what timestamps you want to collect
	// 1 - QueryPerformanceCounter (slower)
	// 3 - RDTSC (faster)
	Properties->Wnode.ClientContext = 1;

	Properties->Wnode.Flags = WNODE_FLAG_TRACED_GUID;
	Properties->BufferSize = 256; // 256KB
	Properties->MinimumBuffers = 4 * SysInfo.dwNumberOfProcessors;
	Properties->MaximumBuffers = Properties->MinimumBuffers + 20;
	Properties->LogFileMode = EVENT_TRACE_REAL_TIME_MODE | EVENT_TRACE_SYSTEM_LOGGER_MODE;
	Properties->LoggerNameOffset = offsetof(FTrace, SessionName);
	Properties->FlushTimer = 1;
	Properties->EnableFlags = 0;

	if ((EnabledEvents & FPlatformEventsTrace::EEventType::ContextSwitch) != FPlatformEventsTrace::EEventType::None)
	{
		Properties->EnableFlags |= EVENT_TRACE_FLAG_CSWITCH;
	}
	if ((EnabledEvents & FPlatformEventsTrace::EEventType::StackSampling) != FPlatformEventsTrace::EEventType::None)
	{
		Properties->EnableFlags |= EVENT_TRACE_FLAG_PROFILE;
	}

	// start the trace
	{
		ULONG Status = ::StartTraceW(&Session, KERNEL_LOGGER_NAMEW, Properties);
		if (Status == ERROR_ACCESS_DENIED)
		{
			UE_LOGF(LogPlatformEvents, Error, "Administrator rights required for ETW");
			return false;
		}
		else if (Status != ERROR_SUCCESS)
		{
			UE_LOGF(LogPlatformEvents, Warning, "Unable to start ETW trace: 0x%08x", Status);
			return false;
		}
	}

	// select for which events ETW should collect stack (StackWalkEvent), this is harmless if stack sampling is not enabled
	{
		// 0x2e -> PERFINFO_LOG_TYPE_SAMPLED_PROFILE
		CLASSIC_EVENT_ID Events[] = { { ETW_PerfInfoGuid, 0x2e } };

		ULONG Status = ::TraceSetInformation(Session, TraceStackTracingInfo, Events, sizeof(Events));
		if (Status != ERROR_SUCCESS)
		{
			UE_LOGF(LogPlatformEvents, Error, "Cannot enable stack tracing for ETW: 0x%08x", Status);
			return false;
		}
	}

	// open trace for receiving callbacks
	{
		EVENT_TRACE_LOGFILEW LogFile;
		ZeroMemory(&LogFile, sizeof(LogFile));
		LogFile.LoggerName = (LPWSTR)KERNEL_LOGGER_NAMEW;
		LogFile.ProcessTraceMode = PROCESS_TRACE_MODE_EVENT_RECORD | PROCESS_TRACE_MODE_RAW_TIMESTAMP | PROCESS_TRACE_MODE_REAL_TIME;
		LogFile.EventRecordCallback = &TraceEventCallback;

		TraceHandle = ::OpenTraceW(&LogFile);
		if (TraceHandle == INVALID_PROCESSTRACE_HANDLE)
		{
			UE_LOGF(LogPlatformEvents, Error, "Cannot open ETW trace");
			return false;
		}
	}

	return true;
}

/////////////////////////////////////////////////////////////////////

void FPlatformEvents::StopETW()
{
	// stop the trace
	::ControlTraceW(0, KERNEL_LOGGER_NAMEW, &Trace.Properties, EVENT_TRACE_CONTROL_STOP);

	if (TraceHandle != INVALID_PROCESSTRACE_HANDLE)
	{
		// close processing loop, this will wait until all pending buffers are flushed
		// and TracingCallback called on all pending events in buffers
		::CloseTrace(TraceHandle);
		TraceHandle = INVALID_PROCESSTRACE_HANDLE;
	}
}

/////////////////////////////////////////////////////////////////////

void FPlatformEventsTrace::Init(uint32 SamplingIntervalUsec)
{
	FPlatformEventsTrace::OutputSettings(SamplingIntervalUsec);

	GPlatformEvents = new FPlatformEvents(SamplingIntervalUsec);
}

bool FPlatformEventsTrace::CanEnable(FPlatformEventsTrace::EEventType Event, const TCHAR** OutReason)
{
	if (GPlatformEvents)
	{
		return GPlatformEvents->CanEnable(Event, OutReason);
	}
	return false;
}

void FPlatformEventsTrace::Enable(FPlatformEventsTrace::EEventType Event)
{
	if (GPlatformEvents)
	{
		GPlatformEvents->Enable(Event);
	}
}

void FPlatformEventsTrace::Disable(FPlatformEventsTrace::EEventType Event)
{
	if (GPlatformEvents)
	{
		GPlatformEvents->Disable(Event);
	}
}

void FPlatformEventsTrace::Stop()
{
	if (GPlatformEvents)
	{
		delete GPlatformEvents;
		GPlatformEvents = nullptr;
	}
}

#endif // PLATFORM_SUPPORTS_PLATFORM_EVENTS
