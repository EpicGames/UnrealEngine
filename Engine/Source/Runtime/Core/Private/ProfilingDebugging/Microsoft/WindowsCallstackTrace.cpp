// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProfilingDebugging/CallstackTracePrivate.h"

#if defined(PLATFORM_SUPPORTS_TRACE_WIN32_CALLSTACK) && UE_CALLSTACK_TRACE_ENABLED

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "CoreTypes.h"
#include "HAL/CriticalSection.h"
#include "HAL/MemoryBase.h"
#include "HAL/RunnableThread.h"
#include "HAL/LowLevelMemTracker.h"
#include "HAL/PlatformStackWalk.h"
#include "ProfilingDebugging/CallstackTracePrivate.h"
#include "ProfilingDebugging/MemoryTrace.h"

#include "Windows/AllowWindowsPlatformTypes.h"
#	include <winnt.h>
#	include <winternl.h>
#include "Windows/HideWindowsPlatformTypes.h"
#include "Misc/CoreDelegates.h"
#include "Misc/ScopeExit.h"
#include "Trace/Trace.inl"
#include "HAL/Runnable.h"
#include "HAL/PlatformProcess.h"
#include "Misc/ScopeLock.h"
#include "Experimental/Containers/GrowOnlyLockFreeHash.h"

#include <atomic>

#ifndef UE_CALLSTACK_TRACE_FULL_CALLSTACKS
	#define UE_CALLSTACK_TRACE_FULL_CALLSTACKS 0
#endif

// 0=off, 1=stats, 2=validation, 3=truth_compare
#define BACKTRACE_DBGLVL 0

#define BACKTRACE_LOCK_FREE (1 && (BACKTRACE_DBGLVL == 0))

static bool GFullBacktraces = UE_CALLSTACK_TRACE_FULL_CALLSTACKS;
static bool GModulesAreInitialized = false;

// This implementation is using unwind tables which is results in very fast
// stack walking. In some cases this is not suitable, and we then fall back
// to the standard stack walking implementation.
#if !defined(UE_CALLSTACK_TRACE_USE_UNWIND_TABLES)
	// Clang uses physical frame pointer for exception handling, causing
	// many functions (e.g. FMemory::Allocate) to be marked with UWOP_SET_FPREG
	// making callstacks unsuitable for allocation tracing.
	#if PLATFORM_COMPILER_CLANG && !PLATFORM_EXCEPTIONS_DISABLED
		#define UE_CALLSTACK_TRACE_USE_UNWIND_TABLES 0
	#else
		#define UE_CALLSTACK_TRACE_USE_UNWIND_TABLES 1
	#endif
#endif

// ARM64 and ARM64EC must use fallback paths. Unwind tables for those architectures have not been implemented.
#if UE_CALLSTACK_TRACE_USE_UNWIND_TABLES && PLATFORM_CPU_ARM_FAMILY
	#undef UE_CALLSTACK_TRACE_USE_UNWIND_TABLES
	#define UE_CALLSTACK_TRACE_USE_UNWIND_TABLES 0
#endif

#if UE_CALLSTACK_TRACE_USE_UNWIND_TABLES

/*
 * Windows' x64 binaries contain a ".pdata" section that describes the location
 * and size of its functions and details on how to unwind them. The unwind
 * information includes descriptions about a function's stack frame size and
 * the non-volatile registers it pushes onto the stack. From this we can
 * calculate where a call instruction wrote its return address. This is enough
 * to walk the callstack and by caching this information it can be done
 * efficiently.
 *
 * Some functions need a variable amount of stack (such as those that use
 * alloc() for example) will use a frame pointer. Frame pointers involve saving
 * and restoring the stack pointer in the function's prologue/epilogue. This
 * frees the function up to modify the stack pointer arbitrarily. This
 * significantly complicates establishing where a return address is, so this
 * pdata scheme of walking the stack just doesn't support functions like this.
 * Walking stops if it encounters such a function. Fortunately there are
 * usually very few such functions, saving us from having to read and track
 * non-volatile registers which adds a significant amount of work.
 *
 * A further optimisation is to to assume we are only interested methods that
 * are part of engine or game code. As such we only build lookup tables for
 * such modules and never accept OS or third party modules. Backtracing stops
 * if an address is encountered which doesn't map to a known module.
 */

////////////////////////////////////////////////////////////////////////////////
static uint32 AddressToId(UPTRINT Address)
{
	return uint32(Address >> 16);
}

static UPTRINT IdToAddress(uint32 Id)
{
	return static_cast<uint32>(UPTRINT(Id) << 16);
}

struct FIdPredicate
{
	template <class T> bool operator () (uint32 Id, const T& Item) const { return Id < Item.Id; }
	template <class T> bool operator () (const T& Item, uint32 Id) const { return Item.Id < Id; }
};

////////////////////////////////////////////////////////////////////////////////
struct FUnwindInfo
{
	uint8	Version : 3;
	uint8	Flags : 5;
	uint8	PrologBytes;
	uint8	NumUnwindCodes;
	uint8	FrameReg	: 4;
	uint8	FrameRspBias : 4;
};

struct FUnwindCode
{
	uint8	PrologOffset;
	uint8	OpCode : 4;
	uint8	OpInfo : 4;
	uint16	Params[];
};

enum
{
	UWOP_PUSH_NONVOL		= 0,	// 1 node
	UWOP_ALLOC_LARGE		= 1,	// 2 or 3 nodes
	UWOP_ALLOC_SMALL		= 2,	// 1 node
	UWOP_SET_FPREG			= 3,	// 1 node
	UWOP_SAVE_NONVOL		= 4,	// 2 nodes
	UWOP_SAVE_NONVOL_FAR	= 5,	// 3 nodes
	UWOP_EPILOG				= 6,	// 2 nodes
	UWOP_SPARE_CODE			= 7,	// 3 nodes
	UWOP_SAVE_XMM128		= 8,	// 2 nodes
	UWOP_SAVE_XMM128_FAR	= 9,	// 3 nodes
	UWOP_PUSH_MACHFRAME		= 10,	// 1 node
};

////////////////////////////////////////////////////////////////////////////////
class FBacktracer
{
public:
							FBacktracer(FMalloc* InMalloc);
							~FBacktracer();
	static FBacktracer*		Get();
	void					AddModule(UPTRINT Base, const TCHAR* Name);
	void					RemoveModule(UPTRINT Base);
	uint32					GetBacktraceId(UPTRINT ProgCounter, const UPTRINT* StackPtr, uint32 FrameSkip=1);

private:
	struct FFunction
	{
		uint32				Id;
		int32				RspBias;
		uint32				End;
#if BACKTRACE_DBGLVL >= 2
		const FUnwindInfo*	UnwindInfo;
#endif
	};

	struct FModule
	{
		uint32				Id;
		uint32				IdSize;
		uint32				NumFunctions;
#if BACKTRACE_DBGLVL >= 1
		uint16				NumFpTypes;
		//uint16			*padding*
#else
		//uint32			*padding*
#endif
		FFunction*			Functions;
	};

	struct FLookupState
	{
		FModule				Module;
	};

	struct FFunctionLookupSetEntry
	{
		// Bottom 48 bits are key (pointer), top 16 bits are data (RSP bias for function)
		std::atomic_uint64_t Data;

		inline uint64 GetKey() const { return Data.load(std::memory_order_relaxed) & 0xffffffffffffull; }
		inline int32 GetValue() const { return static_cast<int64>(Data.load(std::memory_order_relaxed)) >> 48; }
		inline bool IsEmpty() const { return Data.load(std::memory_order_relaxed) == 0; }
		inline void SetKeyValue(uint64 Key, int32 Value)
		{
			Data.store(Key | (static_cast<int64>(Value) << 48), std::memory_order_relaxed);
		}
		static inline uint32 KeyHash(uint64 Key)
		{
			// 64 bit pointer to 32 bit hash
			Key = (~Key) + (Key << 21);
			Key = Key ^ (Key >> 24);
			Key = Key * 265;
			Key = Key ^ (Key >> 14);
			Key = Key * 21;
			Key = Key ^ (Key >> 28);
			Key = Key + (Key << 31);
			return static_cast<uint32>(Key);
		}
		static void ClearEntries(FFunctionLookupSetEntry* Entries, int32 EntryCount)
		{
			memset(Entries, 0, EntryCount * sizeof(FFunctionLookupSetEntry));
		}
	};
	typedef TGrowOnlyLockFreeHash<FFunctionLookupSetEntry, uint64, int32> FFunctionLookupSet;

	void					ReplaceModule(TArrayView<FModule> ModulesView, FModule& Module, void (*FreeFunc)(void*, void*), void* FreeParam);
	const FFunction*		LookupFunction(UPTRINT Address, FLookupState& State) const;
	static FBacktracer*		Instance;
	mutable FCriticalSection Lock;
	FModule*				Modules;
	int32					ModulesNum;
	int32					ModulesCapacity;
	FMalloc*				Malloc;
	FCallstackTracer		CallstackTracer;
#if BACKTRACE_LOCK_FREE
	mutable FFunctionLookupSet	FunctionLookups;
	mutable bool				bReentranceCheck = false;
#endif
#if BACKTRACE_DBGLVL >= 1
	mutable uint32			NumFpTruncations = 0;
	mutable uint32			TotalFunctions = 0;
#endif
};

////////////////////////////////////////////////////////////////////////////////
FBacktracer* FBacktracer::Instance = nullptr;

////////////////////////////////////////////////////////////////////////////////
FBacktracer::FBacktracer(FMalloc* InMalloc)
	: Malloc(InMalloc)
	, CallstackTracer(InMalloc)
#if BACKTRACE_LOCK_FREE
	, FunctionLookups(InMalloc)
#endif
{
#if BACKTRACE_LOCK_FREE
	FunctionLookups.Reserve(512 * 1024);		// 4 MB
#endif
	ModulesCapacity = 8;
	ModulesNum = 0;
	Modules = (FModule*)Malloc->Malloc(sizeof(FModule) * ModulesCapacity);

	Instance = this;
}

////////////////////////////////////////////////////////////////////////////////
FBacktracer::~FBacktracer()
{
	TArrayView<FModule> ModulesView(Modules, ModulesNum);
	for (FModule& Module : ModulesView)
	{
		Malloc->Free(Module.Functions);
	}
}

////////////////////////////////////////////////////////////////////////////////
FBacktracer* FBacktracer::Get()
{
	return Instance;
}

////////////////////////////////////////////////////////////////////////////////
void FBacktracer::ReplaceModule(
	TArrayView<FModule> ModulesView, FModule& Module,
	void (*FreeFunc)(void*, void*), void* FreeParam)
{
	FModule& Existing = ModulesView[0];

	// Is the module being added a dulplicate? It has been unloaded and reloaded
	// again into the same region
	if (Module.Id == Existing.Id
		&& Module.IdSize == Existing.IdSize
		&& Module.NumFunctions == Existing.NumFunctions)
	{
		FreeFunc(Module.Functions, FreeParam);
		return;
	}

	// Otherwise the module is new and located in a region previously occupied
	// by one or more modules that have since been unloaded.
	FreeFunc(Existing.Functions, FreeParam);
	Existing = Module;

	uint32 IdEnd = Module.Id + Module.IdSize;
	for (FModule& Iter : ModulesView.Mid(1))
	{
		if (Iter.Id >= IdEnd)
		{
			break;
		}

		FreeFunc(Iter.Functions, FreeParam);

		Iter.Id = 0;
		Iter.IdSize = 0;
		Iter.Functions = nullptr;
	}
}

////////////////////////////////////////////////////////////////////////////////
void FBacktracer::AddModule(UPTRINT ModuleBase, const TCHAR* Name)
{
	const auto* DosHeader = (IMAGE_DOS_HEADER*)ModuleBase;
	const auto* NtHeader = (IMAGE_NT_HEADERS*)(ModuleBase + DosHeader->e_lfanew);
	const IMAGE_FILE_HEADER* FileHeader = &(NtHeader->FileHeader);

	if (!GFullBacktraces)
	{
		FStringView NameView(Name);
		if (!NameView.EndsWith(TEXTVIEW(".exe")) && CallstackTrace_FilterModule(NameView))
		{
			return;
		}
	}

	const auto* DataDirectories = NtHeader->OptionalHeader.DataDirectory;
	const auto* ExceptionDirectory = &DataDirectories[IMAGE_DIRECTORY_ENTRY_EXCEPTION];

	if (UPTRINT(&(NtHeader->OptionalHeader)) + FileHeader->SizeOfOptionalHeader < UPTRINT(ExceptionDirectory + 1))
	{
		// Make sure ExceptionDirectory is in PE headers
		return;
	}

	if (ExceptionDirectory->VirtualAddress == 0 || ExceptionDirectory->Size == 0)
	{
		return;
	}

	uint32 NumSections = FileHeader->NumberOfSections;
	const auto* Sections = (IMAGE_SECTION_HEADER*)(UPTRINT(&(NtHeader->OptionalHeader)) + FileHeader->SizeOfOptionalHeader);

	const RUNTIME_FUNCTION* FunctionTables = nullptr;
	uint32 NumFunctions = 0;

	for (uint32 i = 0; i < NumSections; ++i)
	{
		const IMAGE_SECTION_HEADER* Section = Sections + i;
		uint64 SectionAddress = Section->VirtualAddress;
		uint64 SectionSize = Section->Misc.VirtualSize;

		// Find section where RUNTIME_FUNCTION table starts in
		if (ExceptionDirectory->VirtualAddress >= SectionAddress && ExceptionDirectory->VirtualAddress < SectionAddress + SectionSize)
		{
			// And make sure RUNTIME_FUNCTION table fully fits into section
			if ((uint64)ExceptionDirectory->VirtualAddress + ExceptionDirectory->Size <= SectionAddress + SectionSize)
			{
				FunctionTables = (RUNTIME_FUNCTION*)(ModuleBase + ExceptionDirectory->VirtualAddress);
				NumFunctions = uint32(ExceptionDirectory->Size) / sizeof(RUNTIME_FUNCTION);
				break;
			}

			// Otherwise do not use it
			return;
		}
	}

	if (!FunctionTables || NumFunctions == 0)
	{
		return;
	}

	// Allocate some space for the module's function-to-frame-size table
	auto* OutTable = (FFunction*)Malloc->Malloc(sizeof(FFunction) * NumFunctions);
	FFunction* OutTableCursor = OutTable;

	// Extract frame size for each function from pdata's unwind codes.
	uint32 NumFpFuncs = 0;
	for (uint32 i = 0; i < NumFunctions; ++i)
	{
		const RUNTIME_FUNCTION* FunctionTable = FunctionTables + i;

		UPTRINT UnwindInfoAddr = ModuleBase + FunctionTable->UnwindInfoAddress;
		const auto* UnwindInfo = (FUnwindInfo*)UnwindInfoAddr;

		// Unwind infos are two-byte aligned. Unaligned info addresses have been
		// encoutered where the aligned RVA holds a redirecting runtime function
		// and not unwind op codes.
		if (UnwindInfoAddr & 1)
		{
			const RUNTIME_FUNCTION* Redirect = (RUNTIME_FUNCTION*)(UnwindInfoAddr & ~1);
			UnwindInfo = (FUnwindInfo*)(ModuleBase + Redirect->UnwindInfoAddress);
		}

		// Functions can end up split into parts and spread about code segments. In
		// such cases there is a chain of unwind info which must be followed to find
		// the primary info that describes the prologue.
		while (UnwindInfo->Flags & UNW_FLAG_CHAININFO)
		{
			uint32 CodeStep = (UnwindInfo->NumUnwindCodes + 1) & ~1;
			const void* OuterAddr = (FUnwindCode*)(UnwindInfo + 1) + CodeStep;
			const RUNTIME_FUNCTION* Outer = (RUNTIME_FUNCTION*)OuterAddr;
			UnwindInfo = (FUnwindInfo*)(ModuleBase + Outer->UnwindInfoAddress);
		}

		// Expect version 1 or 2 (2 introduced the Epilog code).
		if (uint32 Version = UnwindInfo->Version; Version != 1 && Version != 2)
		{
			continue;
		}

		int32 FpInfo = 0;
		int32 RspBias = 0;

#if BACKTRACE_DBGLVL >= 2
		uint32 PrologVerify = UnwindInfo->PrologBytes;
#endif

		const auto* Code = (FUnwindCode*)(UnwindInfo + 1);
		const auto* EndCode = Code + UnwindInfo->NumUnwindCodes;
		bool bEncounteredUnknownOpCode = false;
		while (Code < EndCode && !bEncounteredUnknownOpCode)
		{
#if BACKTRACE_DBGLVL >= 2
			if (Code->OpCode != UWOP_EPILOG)
			{
				// PrologOffset has a different purpose in v2 epilog ops
				if (Code->PrologOffset > PrologVerify)
				{
					PLATFORM_BREAK();
				}
				PrologVerify = Code->PrologOffset;
			}
#endif

			switch (Code->OpCode)
			{
			case UWOP_PUSH_NONVOL:
				RspBias += 8;
				Code += 1;
				break;

			case UWOP_ALLOC_LARGE:
				if (Code->OpInfo)
				{
					RspBias += *(uint32*)(Code->Params);
					Code += 3;
				}
				else
				{
					RspBias += Code->Params[0] * 8;
					Code += 2;
				}
				break;

			case UWOP_ALLOC_SMALL:
				RspBias += (Code->OpInfo * 8) + 8;
				Code += 1;
				break;

			case UWOP_SET_FPREG:
				// Function will adjust RSP (e.g. through use of alloca()) so it
				// uses a frame pointer register. There's instructions like;
				//
				//   push FRAME_REG
				//   lea FRAME_REG, [rsp + (FRAME_RSP_BIAS * 16)]
				//   ...
				//   add rsp, rax
				//   ...
				//   sub rsp, FRAME_RSP_BIAS * 16
				//   pop FRAME_REG
				//   ret
				//
				// To recover the stack frame we would need to track non-volatile
				// registers which adds a lot of overhead for a small subset of
				// functions. Instead we'll end backtraces at these functions.


				// MSB is set to detect variable sized frames that we can't proceed
				// past when back-tracing.
				NumFpFuncs++;
				FpInfo |= 0x80000000 | (uint32(UnwindInfo->FrameReg) << 27) | (uint32(UnwindInfo->FrameRspBias) << 23);
				Code += 1;
				break;

			case UWOP_PUSH_MACHFRAME:
				RspBias = Code->OpInfo ? 48 : 40;
				Code += 1;
				break;

			case UWOP_EPILOG:
				// The epilog op replaced an obsolete save-xmm op. Save-xmm was two codes
				// long. Epilog retained this and is always an even code count. +2 will work
				// even if epilog is three ops long as the opcode value is always present.
				Code += 2;
				break;

			case UWOP_SAVE_NONVOL:		Code += 2; break; /* saves are movs instead of pushes */
			case UWOP_SAVE_NONVOL_FAR:	Code += 3; break;
			case UWOP_SAVE_XMM128:		Code += 2; break;
			case UWOP_SAVE_XMM128_FAR:	Code += 3; break;
			case UWOP_SPARE_CODE:		Code += 3; break;

			default:
#if BACKTRACE_DBGLVL >= 2
				PLATFORM_BREAK();
#endif
				bEncounteredUnknownOpCode = true;
				break;
			}
		}

		RspBias /= sizeof(void*);	// stack push/popds in units of one machine word
		RspBias += 1;				// and one extra push for the ret address
		RspBias |= FpInfo;			// pack in details about possible frame pointer

		*OutTableCursor = {
			.Id = FunctionTable->BeginAddress,
			.RspBias = RspBias,
			.End = FunctionTable->EndAddress,
#if BACKTRACE_DBGLVL >= 2
			.UnwindInfo = UnwindInfo,
#endif
		};

		++OutTableCursor;
	}

	UPTRINT ModuleSize = NtHeader->OptionalHeader.SizeOfImage;
	ModuleSize += 0xffff; // to align up to next 64K page. it'll get shifted by AddressToId()

	FModule Module = {
		AddressToId(ModuleBase),
		AddressToId(ModuleSize),
		uint32(UPTRINT(OutTableCursor - OutTable)),
#if BACKTRACE_DBGLVL >= 1
		uint16(NumFpFuncs),
#endif
		OutTable,
	};

	struct FDeferredFrees
	{
		enum : uint32 { Max = 8 };
		void*	Addresses[Max];
		uint32	Num = 0;
		void	Defer(void* Addr) { if (Num < Max) Addresses[Num++] = Addr; }
	};

	FDeferredFrees DeferredFrees;
	ON_SCOPE_EXIT
	{
		for (uint32 i = 0; i < DeferredFrees.Num; ++i)
		{
			Malloc->Free(DeferredFrees.Addresses[i]);
		}
	};

	{
		FScopeLock _(&Lock);

		ON_SCOPE_EXIT
		{
			// For ease of early outs and logical paragraphs, we'll sort out the
			// way out of the locked scope.
			auto Predicate = [] (const FModule& A, const FModule& B)
			{
				return A.Id < B.Id;
			};
			Algo::Sort(TArrayView<FModule>(Modules, ModulesNum), Predicate);
		};

		TArrayView<FModule> ModulesView(Modules, ModulesNum);
		int32 Index = Algo::UpperBound(ModulesView, Module.Id, FIdPredicate()) - 1;
		if (Index >= 0)
		{
			auto FreeThunk = [] (void* AddrToFree, void* Param)
			{
				((FDeferredFrees*)Param)->Defer(AddrToFree);
			};

			// Move the view so it starts at the first module of interest.
			ModulesView = ModulesView.Mid(Index);

			// Is the new module's starting address (Id) contained within an
			// existing unloaded module?
			uint32 IdEnd = ModulesView[0].Id + ModulesView[0].IdSize;
			if (Module.Id < IdEnd)
			{
				ReplaceModule(ModulesView, Module, FreeThunk, &DeferredFrees);
				return;
			}

			// Or perhaps the new module ends in the address space of the next
			// module along (which has since been unloaded)?
			ModulesView = ModulesView.Mid(1);
			if (!ModulesView.IsEmpty() && (Module.Id + Module.IdSize) > ModulesView[0].Id)
			{
				ReplaceModule(ModulesView, Module, FreeThunk, &DeferredFrees);
				return;
			}
		}

		if (ModulesNum + 1 > ModulesCapacity)
		{
			ModulesCapacity += 32;
			Modules = (FModule*)Malloc->Realloc(Modules, sizeof(FModule) * ModulesCapacity);
		}
		Modules[ModulesNum++] = Module;
	}

#if BACKTRACE_DBGLVL >= 1
	NumFpTruncations += NumFpFuncs;
	TotalFunctions += NumFunctions;
#endif
}

////////////////////////////////////////////////////////////////////////////////
void FBacktracer::RemoveModule(UPTRINT ModuleBase)
{
	// When Windows' RequestExit() is called it hard-terminates all threads except
	// the main thread and then proceeds to unload the process' DLLs. This hard 
	// thread termination can result is dangling locked locks. Not an issue as
	// the rule is "do not do anything multithreaded in DLL load/unload". And here
	// we are, taking write locks during DLL unload which is, quite unsurprisingly,
	// deadlocking. In reality tracking Windows' DLL unloads doesn't tell us
	// anything due to how DLLs and processes' address spaces work. So we will...
#if defined PLATFORM_WINDOWS
	return;
#else

	FScopeLock _(&Lock);

	uint32 ModuleId = AddressToId(ModuleBase);
	TArrayView<FModule> ModulesView(Modules, ModulesNum);
	int32 Index = Algo::LowerBound(ModulesView, ModuleId, FIdPredicate());
	if (Index >= ModulesNum)
	{
		return;
	}

	const FModule& Module = Modules[Index];
	if (Module.Id != ModuleId)
	{
		return;
	}

#if BACKTRACE_DBGLVL >= 1
	NumFpTruncations -= Module.NumFpTypes;
	TotalFunctions -= Module.NumFunctions;
#endif

	// no code should be executing at this point so we can safely free the
	// table knowing know one is looking at it.
	Malloc->Free(Module.Functions);

	for (SIZE_T i = Index; i < ModulesNum; i++)
	{
		Modules[i] = Modules[i + 1];
	}

	--ModulesNum;
#endif
}

////////////////////////////////////////////////////////////////////////////////
const FBacktracer::FFunction* FBacktracer::LookupFunction(UPTRINT Address, FLookupState& State) const
{
	// This function caches the previous module look up. The theory here is that
	// a series of return address in a backtrace often cluster around one module

	FIdPredicate IdPredicate;

	// Look up the module that Address belongs to.
	uint32 AddressId = AddressToId(Address);
	if ((AddressId - State.Module.Id) >= State.Module.IdSize)
	{
		TArrayView<FModule> ModulesView(Modules, ModulesNum);
		uint32 Index = Algo::UpperBound(ModulesView, AddressId, IdPredicate);
		if (Index == 0)
		{
			return nullptr;
		}

		State.Module = Modules[Index - 1];
	}

	// Check that the address is within the address space of the best-found module
	const FModule* Module = &(State.Module);
	if ((AddressId - Module->Id) >= Module->IdSize)
	{
		return nullptr;
	}

	// Now we've a module we have a table of functions and their stack sizes so
	// we can get the frame size for Address
	uint32 FuncId = uint32(Address - IdToAddress(Module->Id)); 
	TArrayView<FFunction> FuncsView(Module->Functions, Module->NumFunctions);
	uint32 Index = Algo::UpperBound(FuncsView, FuncId, IdPredicate);
	if (Index == 0)
	{
		return nullptr;
	}

	const FFunction* Function = Module->Functions + (Index - 1);
	if (FuncId >= Function->End)
	{
		// The lookup fell outside of functions with stack frames so we will return
		// a generic result that gives a stack frame size of 0 (+1 for return addr).
		static FFunction NoFrameFunction = {
			.RspBias = 1,
		};
		return &NoFrameFunction;
	}

#if BACKTRACE_DBGLVL >= 2
	if ((FuncId - Function->Id) >= Function->End - Function->Id)
	{
		PLATFORM_BREAK();
		return nullptr;
	}
#endif

	return Function;
}

////////////////////////////////////////////////////////////////////////////////
uint32 FBacktracer::GetBacktraceId(UPTRINT ProgCounter, const UPTRINT* StackPointer, uint32 FrameSkip) 
{
	FLookupState LookupState = {};
	uint64 Frames[UE_CALLSTACK_TRACE_MAX_FRAMES];

#if BACKTRACE_DBGLVL >= 3
	UPTRINT TruthBacktrace[1024];
	uint32 NumTruth = RtlCaptureStackBackTrace(0, 1024, (void**)TruthBacktrace, nullptr);
	UPTRINT* TruthCursor = TruthBacktrace;
	for (; *TruthCursor != *StackPointer; ++TruthCursor);
#endif

#if BACKTRACE_DBGLVL >= 2
	struct { void* Sp; void* Ip; const FFunction* Function; } Backtrace[1024] = {};
	uint32 NumBacktrace = 0;
#endif

	uint64 BacktraceHash = 0;
	uint32 FrameIdx = 0;

	auto AddFrame = [&] (int32 RspBias)
	{
		StackPointer += RspBias;
		ProgCounter = StackPointer[-1];
		if (ProgCounter == 0)
		{
			return false;
		}

		if (FrameSkip != 0)
		{
			--FrameSkip;
			return true;
		}

		// This is a simple order-dependent LCG. Should be sufficient enough
		BacktraceHash += ProgCounter;
		BacktraceHash *= 0x30be8efa499c249dull;

		Frames[FrameIdx++] = ProgCounter;
		return FrameIdx < UE_ARRAY_COUNT(Frames);
	};

#if BACKTRACE_LOCK_FREE
	// When running lock free, we defer the lock until a lock free function lookup fails
	bool Locked = false;
#else
	FScopeLock _(&Lock);
#endif
	while (true)
	{
		UPTRINT RetAddr = ProgCounter;

#if BACKTRACE_LOCK_FREE
		int32 RspBias;
		bool bIsAlreadyInTable;
		FunctionLookups.Find(RetAddr, &RspBias, &bIsAlreadyInTable);
		if (bIsAlreadyInTable)
		{
			if (RspBias < 0)
			{
				break;
			}
			else if (!AddFrame(RspBias))
			{
				break;
			}
			continue;
		}
		if (!Locked)
		{
			Lock.Lock();
			Locked = true;

			// If FunctionLookups.Emplace triggers a reallocation, it can cause an infinite recursion
			// when the allocation reenters the stack trace code.  We need to break out of the recursion
			// in that case, and let the allocation complete, with the assumption that we don't care
			// about call stacks for internal allocations in the memory reporting system.  The "Lock()"
			// above will only fall through with this flag set if it's a second lock in the same thread.
			if (bReentranceCheck)
			{
				break;
			}
		}
#endif  // BACKTRACE_LOCK_FREE

		const FFunction* Function = LookupFunction(RetAddr, LookupState);
		if (Function == nullptr)
		{
#if BACKTRACE_LOCK_FREE
			// LookupFunction fails when modules are not yet registered. In this case, we do not want the address
			// to be added to the lookup map, but to retry the lookup later when modules are properly registered.
			if (GModulesAreInitialized)
			{
				TGuardValue<bool> ReentranceGuard(bReentranceCheck, true);
				FunctionLookups.Emplace(RetAddr, -1);
			}
#endif
			break;
		}

#if BACKTRACE_LOCK_FREE
		{
			// This conversion improves probing performance for the hash set. Additionally it is critical 
			// to avoid incorrect values when RspBias is compressed into 16 bits in the hash map.
			int32 StoreBias = Function->RspBias < 0 ? -1 : Function->RspBias;
			TGuardValue<bool> ReentranceGuard(bReentranceCheck, true);
			FunctionLookups.Emplace(RetAddr, StoreBias);
		}
#endif

#if BACKTRACE_DBGLVL >= 2
		if (NumBacktrace < 1024)
		{
			Backtrace[NumBacktrace++] = {
				StackPointer,
				(void*)RetAddr,
				Function,
			};
		}
#endif

		if (Function->RspBias < 0)
		{
			// This is a frame with a variable-sized stack pointer. We don't
			// track enough information to proceed.
#if BACKTRACE_DBGLVL >= 1
			NumFpTruncations++;
#endif
			break;
		}

		if (!AddFrame(Function->RspBias))
		{
			break;
		}
	}

	// Build the backtrace entry for submission
	FCallstackTracer::FBacktraceEntry BacktraceEntry;
	BacktraceEntry.Hash = BacktraceHash;
	BacktraceEntry.FrameCount = FrameIdx;
	BacktraceEntry.Frames = Frames;

#if BACKTRACE_DBGLVL >= 3
	for (uint32 i = 0; i < NumBacktrace; ++i)
	{
		if ((void*)TruthCursor[i] != Backtrace[i].Ip)
		{
			PLATFORM_BREAK();
			break;
		}
	}
#endif

#if BACKTRACE_LOCK_FREE
	if (Locked)
	{
		Lock.Unlock();
	}
#endif
	// Add to queue to be processed. This might block until there is room in the
	// queue (i.e. the processing thread has caught up processing).
	return CallstackTracer.AddCallstack(BacktraceEntry);
}

#else // UE_CALLSTACK_TRACE_USE_UNWIND_TABLES

////////////////////////////////////////////////////////////////////////////////
class FBacktracer
{
public:
	FBacktracer(FMalloc* InMalloc);
	~FBacktracer();
	static FBacktracer*	Get();
	inline uint32 GetBacktraceId(const void* AddressOfReturnAddress, uint32 FrameSkip);
	uint32 GetBacktraceId(uint64 ReturnAddress, uint32 FrameSkip);
	void AddModule(UPTRINT Base, const TCHAR* Name) {}
	void RemoveModule(UPTRINT Base) {}

private:
	static FBacktracer* Instance;
	FMalloc* Malloc;
	FCallstackTracer CallstackTracer;
};

////////////////////////////////////////////////////////////////////////////////
FBacktracer* FBacktracer::Instance = nullptr;

////////////////////////////////////////////////////////////////////////////////
FBacktracer::FBacktracer(FMalloc* InMalloc)
	: Malloc(InMalloc)
	, CallstackTracer(InMalloc)
{
	Instance = this;
}

////////////////////////////////////////////////////////////////////////////////
FBacktracer::~FBacktracer()
{
}

////////////////////////////////////////////////////////////////////////////////
FBacktracer* FBacktracer::Get()
{
	return Instance;
}

////////////////////////////////////////////////////////////////////////////////
uint32 FBacktracer::GetBacktraceId(const void* AddressOfReturnAddress, uint32 FrameSkip)
{
	const uint64 ReturnAddress = *(const uint64*)AddressOfReturnAddress;
	return GetBacktraceId(ReturnAddress, FrameSkip);
}

////////////////////////////////////////////////////////////////////////////////
uint32 FBacktracer::GetBacktraceId(uint64 ReturnAddress, uint32 FrameSkip)
{
#if !UE_BUILD_SHIPPING
	uint64 StackFrames[UE_CALLSTACK_TRACE_MAX_FRAMES];
	int32 NumStackFrames = FPlatformStackWalk::CaptureStackBackTrace(StackFrames, UE_ARRAY_COUNT(StackFrames));
	if (NumStackFrames > 0)
	{
		FCallstackTracer::FBacktraceEntry BacktraceEntry;
		uint64 BacktraceId = 0;
		uint32 FrameIdx = 0;
		bool bUseAddress = false;
		for (int32 Index = 0; Index < NumStackFrames; Index++)
		{
			if (!bUseAddress)
			{
				// start using backtrace only after ReturnAddress
				if (StackFrames[Index] == (uint64)ReturnAddress)
				{
					bUseAddress = true;
				}
			}
			if (bUseAddress || NumStackFrames == 1)
			{
				uint64 RetAddr = StackFrames[Index];
				StackFrames[FrameIdx++] = RetAddr;

				// This is a simple order-dependent LCG. Should be sufficient enough
				BacktraceId += RetAddr;
				BacktraceId *= 0x30be8efa499c249dull;
			}
		}

		// Save the collected id
		BacktraceEntry.Hash = BacktraceId;

		if (FrameIdx > FrameSkip)
		{
			BacktraceEntry.FrameCount = FrameIdx - FrameSkip;
			BacktraceEntry.Frames = StackFrames + FrameSkip;
		}
		else
		{
			BacktraceEntry.FrameCount = FrameIdx;
			BacktraceEntry.Frames = StackFrames;
		}

		// Add to queue to be processed. This might block until there is room in the
		// queue (i.e. the processing thread has caught up processing).
		return CallstackTracer.AddCallstack(BacktraceEntry);
	}
#endif

	return 0;
}
#endif // UE_CALLSTACK_TRACE_USE_UNWIND_TABLES

////////////////////////////////////////////////////////////////////////////////
void Modules_Create(FMalloc*);
void Modules_Subscribe(void (*)(bool, void*, const TCHAR*));
void Modules_Initialize();

////////////////////////////////////////////////////////////////////////////////
void CallstackTrace_CreateInternal(FMalloc* Malloc)
{

	if (FBacktracer::Get() != nullptr)
	{
		return;
	}

	// Allocate, construct and intentionally leak backtracer
	void* Alloc = Malloc->Malloc(sizeof(FBacktracer), alignof(FBacktracer));
	new (Alloc) FBacktracer(Malloc);

	const TCHAR* CmdLine = ::GetCommandLineW();
	if (const TCHAR* TraceArg = FCString::Stristr(CmdLine, TEXT("-tracefullcallstacks")))
	{
		GFullBacktraces = true;
	}

	Modules_Create(Malloc);
	Modules_Subscribe(
		[] (bool bLoad, void* Module, const TCHAR* Name)
		{
			bLoad
				? FBacktracer::Get()->AddModule(UPTRINT(Module), Name) //-V522
				: FBacktracer::Get()->RemoveModule(UPTRINT(Module));
		}
	);
}

////////////////////////////////////////////////////////////////////////////////
void CallstackTrace_InitializeInternal()
{
	Modules_Initialize();
	GModulesAreInitialized = true;
}

////////////////////////////////////////////////////////////////////////////////
#if PLATFORM_WINDOWS
bool CallstackTrace_FilterModule(FStringView ModuleName)
{
	return !ModuleName.Contains(TEXTVIEW("Binaries")) && ModuleName.Contains(TEXTVIEW("ThirdParty"));
}
#endif

////////////////////////////////////////////////////////////////////////////////
FORCENOINLINE static const UPTRINT* CallstackTrace_GetStackFrame()
{
	// Returns the stack frame's address for the caller.

#if defined(__clang__)
	// Using __builtin_frame_address(0) will cause this function to get a frame
	// pointer, which takes one push on the stack. As the function will not be
	// inlined, there is a return address too - another push. So the bias adjust
	// the stack frame to match the caller.
	enum { Bias = 2 };
	return (UPTRINT*)__builtin_frame_address(0) + Bias;

#elif defined(_MSC_VER)
	// _AddressOfReturnAddress() is already pointing at the callers frame, but it
	// is off by one push due to the return address itself. Bias adjusts for that.
	enum { Bias = 1 };
	return (UPTRINT*)_AddressOfReturnAddress() + Bias;

#else
	return nullptr;
#endif
}

////////////////////////////////////////////////////////////////////////////////
FORCENOINLINE static uint32 CallstackTrace_GetCurrentId_Private()
{
	FBacktracer* Instance = FBacktracer::Get();
	if (Instance == nullptr)
	{
		return 0;
	}

	// We don't need the parent function in the traced callstack which will be
	// CallstackTrace_GetCurrentId() or possibly MemoryTrace_Alloc/Free (inlining)
	const uint32 FrameSkip = 1;

#if UE_CALLSTACK_TRACE_USE_UNWIND_TABLES

	const UPTRINT* StackPtr = CallstackTrace_GetStackFrame();
	if (StackPtr == nullptr)
	{
		return 0;
	}
	// It is expected that StackPtr == RSP at this point.

	UPTRINT ProgCounter = UPTRINT(&CallstackTrace_GetCurrentId_Private);
	return Instance->GetBacktraceId(ProgCounter, StackPtr, FrameSkip);

#else

	void* StackAddress = PLATFORM_RETURN_ADDRESS_FOR_CALLSTACKTRACING();

#if PLATFORM_USE_CALLSTACK_ADDRESS_POINTER
		return Instance->GetBacktraceId(StackAddress, FrameSkip);
#else
		return Instance->GetBacktraceId((uint64)StackAddress, FrameSkip);
#endif

#endif
}

////////////////////////////////////////////////////////////////////////////////
uint32 CallstackTrace_GetCurrentId()
{
	if (!UE_TRACE_CHANNELEXPR_IS_ENABLED(CallstackChannel))
	{
		return 0;
	}

	return CallstackTrace_GetCurrentId_Private();
}

#endif // defined(PLATFORM_SUPPORTS_TRACE_WIN32_CALLSTACK) && UE_CALLSTACK_TRACE_ENABLED
