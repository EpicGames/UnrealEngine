// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProfilingDebugging/MemoryTrace.h"

#include "ProfilingDebugging/CallstackTrace.h"
#include "ProfilingDebugging/TraceMalloc.h"

#if UE_MEMORY_TRACE_ENABLED

#include "Containers/StringView.h"
#include "CoreTypes.h"
#include "HAL/MemoryBase.h"
#include "HAL/Platform.h"
#include "HAL/PlatformTime.h"
#include "Misc/CoreDelegates.h"
#include "Misc/CString.h"
#include "Trace/Trace.inl"

#include "Windows/AllowWindowsPlatformTypes.h"
#include <shellapi.h>
#include <winternl.h>
#include <intrin.h>

#if defined(PLATFORM_SUPPORTS_TRACE_WIN32_VIRTUAL_MEMORY_HOOKS)
#include <detours/detours.h>
#endif

#if (NTDDI_VERSION >= NTDDI_WIN10_RS4)
#pragma comment(lib, "mincore.lib") // VirtualAlloc2
#endif

////////////////////////////////////////////////////////////////////////////////
FMalloc* MemoryTrace_CreateInternal(FMalloc*, int32, const WIDECHAR* const*);

////////////////////////////////////////////////////////////////////////////////
struct FAddrPack
{
			FAddrPack() = default;
			FAddrPack(UPTRINT Addr, uint16 Value) { Set(Addr, Value); }
	void	Set(UPTRINT Addr, uint16 Value) { Inner = uint64(Addr) | (uint64(Value) << 48ull); }
	uint64	Inner;
};
static_assert(sizeof(FAddrPack) == sizeof(uint64), "");

////////////////////////////////////////////////////////////////////////////////
#if defined(PLATFORM_SUPPORTS_TRACE_WIN32_VIRTUAL_MEMORY_HOOKS)

class FVirtualWinApiHooks
{
public:
	static void				Initialize(bool bInLight);

private:
							FVirtualWinApiHooks();
	static bool				bLight;
	static LPVOID WINAPI	VmAlloc(LPVOID Address, SIZE_T Size, DWORD Type, DWORD Protect);
	static LPVOID WINAPI	VmAllocEx(HANDLE Process, LPVOID Address, SIZE_T Size, DWORD Type, DWORD Protect);
#if (NTDDI_VERSION >= NTDDI_WIN10_RS4)
	static PVOID WINAPI		VmAlloc2(HANDLE Process, PVOID BaseAddress, SIZE_T Size, ULONG AllocationType,
								ULONG PageProtection, MEM_EXTENDED_PARAMETER* ExtendedParameters, ULONG ParameterCount);
	static PVOID(WINAPI* VmAlloc2Orig)(HANDLE, PVOID, SIZE_T, ULONG, ULONG, MEM_EXTENDED_PARAMETER*, ULONG);
	typedef PVOID(__stdcall* FnVirtualAlloc2)(HANDLE, PVOID, SIZE_T, ULONG, ULONG, MEM_EXTENDED_PARAMETER*, ULONG);
#else
	static LPVOID WINAPI	VmAlloc2(HANDLE Process, LPVOID BaseAddress, SIZE_T Size, ULONG AllocationType,
								ULONG PageProtection, void* ExtendedParameters, ULONG ParameterCount);
	static LPVOID(WINAPI* VmAlloc2Orig)(HANDLE, LPVOID, SIZE_T, ULONG, ULONG, /*MEM_EXTENDED_PARAMETER* */ void*, ULONG);
	typedef LPVOID(__stdcall* FnVirtualAlloc2)(HANDLE, LPVOID, SIZE_T, ULONG, ULONG, /* MEM_EXTENDED_PARAMETER* */ void*, ULONG);
#endif
	static BOOL WINAPI		VmFree(LPVOID Address, SIZE_T Size, DWORD Type);
	static BOOL WINAPI		VmFreeEx(HANDLE Process, LPVOID Address, SIZE_T Size, DWORD Type);
	static LPVOID			(WINAPI *VmAllocOrig)(LPVOID, SIZE_T, DWORD, DWORD);
	static LPVOID			(WINAPI *VmAllocExOrig)(HANDLE, LPVOID, SIZE_T, DWORD, DWORD);
	static BOOL				(WINAPI *VmFreeOrig)(LPVOID, SIZE_T, DWORD);
	static BOOL				(WINAPI *VmFreeExOrig)(HANDLE, LPVOID, SIZE_T, DWORD);

};

////////////////////////////////////////////////////////////////////////////////
bool	FVirtualWinApiHooks::bLight;
LPVOID	(WINAPI *FVirtualWinApiHooks::VmAllocOrig)(LPVOID, SIZE_T, DWORD, DWORD);
LPVOID	(WINAPI *FVirtualWinApiHooks::VmAllocExOrig)(HANDLE, LPVOID, SIZE_T, DWORD, DWORD);
#if (NTDDI_VERSION >= NTDDI_WIN10_RS4)
PVOID	(WINAPI* FVirtualWinApiHooks::VmAlloc2Orig)(HANDLE, PVOID, SIZE_T, ULONG, ULONG, MEM_EXTENDED_PARAMETER*, ULONG);
#else
LPVOID	(WINAPI* FVirtualWinApiHooks::VmAlloc2Orig)(HANDLE, LPVOID, SIZE_T, ULONG, ULONG, /*MEM_EXTENDED_PARAMETER* */ void*, ULONG);
#endif
BOOL	(WINAPI *FVirtualWinApiHooks::VmFreeOrig)(LPVOID, SIZE_T, DWORD);
BOOL	(WINAPI *FVirtualWinApiHooks::VmFreeExOrig)(HANDLE, LPVOID, SIZE_T, DWORD);

////////////////////////////////////////////////////////////////////////////////
void FVirtualWinApiHooks::Initialize(bool bInLight)
{
	bLight = bInLight;

	// Note that hooking alloc functions is done last as applying the hook can
	// allocate some memory pages.

	DetourTransactionBegin();
	DetourUpdateThread(GetCurrentThread());

	VmFreeOrig = VirtualFree;
	DetourAttach(&VmFreeOrig, &FVirtualWinApiHooks::VmFree);

	VmFreeExOrig = VirtualFreeEx;
	DetourAttach(&VmFreeExOrig, &FVirtualWinApiHooks::VmFreeEx);

#if PLATFORM_WINDOWS
#if (NTDDI_VERSION >= NTDDI_WIN10_RS4)
	{
		VmAlloc2Orig = VirtualAlloc2;
		DetourAttach(&VmAlloc2Orig, &FVirtualWinApiHooks::VmAlloc2);
	}
#else // NTDDI_VERSION
	{
		VmAlloc2Orig = nullptr;
		HINSTANCE DllInstance;
		DllInstance = LoadLibrary(TEXT("kernelbase.dll"));
		if (DllInstance != NULL)
		{
PRAGMA_DISABLE_CAST_FUNCTION_TYPE_MISMATCH_WARNINGS
			VmAlloc2Orig = (FnVirtualAlloc2)GetProcAddress(DllInstance, "VirtualAlloc2");
PRAGMA_ENABLE_CAST_FUNCTION_TYPE_MISMATCH_WARNINGS
			FreeLibrary(DllInstance);
		}
		if (VmAlloc2Orig)
		{
			DetourAttach(&VmAlloc2Orig, &FVirtualWinApiHooks::VmAlloc2);
		}
	}
#endif // NTDDI_VERSION
#endif // PLATFORM_WINDOWS

	VmAllocExOrig = VirtualAllocEx;
	DetourAttach(&VmAllocExOrig, &FVirtualWinApiHooks::VmAllocEx);

	VmAllocOrig = VirtualAlloc;
	DetourAttach(&VmAllocOrig, &FVirtualWinApiHooks::VmAlloc);

	LONG Result = DetourTransactionCommit();
	if (Result != NO_ERROR)
	{
		UE_LOGF(LogMemory, Warning, "Memory trace failed to install hooks. Detours returned 0x%08x", Result);
	}
}

////////////////////////////////////////////////////////////////////////////////
LPVOID WINAPI FVirtualWinApiHooks::VmAlloc(LPVOID Address, SIZE_T Size, DWORD Type, DWORD Protect)
{
	LPVOID Ret = VmAllocOrig(Address, Size, Type, Protect);

	// Track any reserve for now. Going forward we need events to differentiate reserves/commits and
	// corresponding information on frees.
	if (Ret != nullptr &&
		((Type & MEM_RESERVE) || ((Type & MEM_COMMIT) && Address == nullptr)))
	{
		MemoryTrace_Alloc((uint64)Ret, Size, 0, EMemoryTraceRootHeap::SystemMemory);
		MemoryTrace_MarkAllocAsHeap((uint64)Ret, EMemoryTraceRootHeap::SystemMemory);
	}

	return Ret;
}

////////////////////////////////////////////////////////////////////////////////
BOOL WINAPI FVirtualWinApiHooks::VmFree(LPVOID Address, SIZE_T Size, DWORD Type)
{
	if (Type & MEM_RELEASE)
	{
		MemoryTrace_UnmarkAllocAsHeap((uint64)Address, EMemoryTraceRootHeap::SystemMemory);
		MemoryTrace_Free((uint64)Address, EMemoryTraceRootHeap::SystemMemory);
	}

	return VmFreeOrig(Address, Size, Type);
}

////////////////////////////////////////////////////////////////////////////////
LPVOID WINAPI FVirtualWinApiHooks::VmAllocEx(HANDLE Process, LPVOID Address, SIZE_T Size, DWORD Type, DWORD Protect)
{
	LPVOID Ret = VmAllocExOrig(Process, Address, Size, Type, Protect);

	if (Process == GetCurrentProcess() && Ret != nullptr &&
		((Type & MEM_RESERVE) || ((Type & MEM_COMMIT) && Address == nullptr)))
	{
		MemoryTrace_Alloc((uint64)Ret, Size, 0, EMemoryTraceRootHeap::SystemMemory);
		MemoryTrace_MarkAllocAsHeap((uint64)Ret, EMemoryTraceRootHeap::SystemMemory);
	}

	return Ret;
}

////////////////////////////////////////////////////////////////////////////////
BOOL WINAPI FVirtualWinApiHooks::VmFreeEx(HANDLE Process, LPVOID Address, SIZE_T Size, DWORD Type)
{
	if (Process == GetCurrentProcess() && (Type & MEM_RELEASE))
	{
		MemoryTrace_UnmarkAllocAsHeap((uint64)Address, EMemoryTraceRootHeap::SystemMemory);
		MemoryTrace_Free((uint64)Address, EMemoryTraceRootHeap::SystemMemory);
	}

	return VmFreeExOrig(Process, Address, Size, Type);
}

////////////////////////////////////////////////////////////////////////////////
#if (NTDDI_VERSION >= NTDDI_WIN10_RS4)
PVOID WINAPI FVirtualWinApiHooks::VmAlloc2(HANDLE Process, PVOID BaseAddress, SIZE_T Size, ULONG Type,
	ULONG PageProtection, MEM_EXTENDED_PARAMETER* ExtendedParameters, ULONG ParameterCount)
#else
LPVOID WINAPI FVirtualWinApiHooks::VmAlloc2(HANDLE Process, LPVOID BaseAddress, SIZE_T Size, ULONG Type,
	ULONG PageProtection, /*MEM_EXTENDED_PARAMETER* */ void* ExtendedParameters, ULONG ParameterCount)
#endif
{
	LPVOID Ret = VmAlloc2Orig(Process, BaseAddress, Size, Type, PageProtection, ExtendedParameters, ParameterCount);

	if (Process == GetCurrentProcess() && Ret != nullptr &&
		((Type & MEM_RESERVE) || ((Type & MEM_COMMIT) && BaseAddress == nullptr)))
	{
		MemoryTrace_Alloc((uint64)Ret, Size, 0, EMemoryTraceRootHeap::SystemMemory);
		MemoryTrace_MarkAllocAsHeap((uint64)Ret, EMemoryTraceRootHeap::SystemMemory);
	}

	return Ret;
}
#endif // defined(PLATFORM_SUPPORTS_TRACE_WIN32_VIRTUAL_MEMORY_HOOKS)

////////////////////////////////////////////////////////////////////////////////
FMalloc* MemoryTrace_Create(FMalloc* InMalloc)
{
	bool bEnabled = false;

	int ArgC = 0;
	const WIDECHAR* CmdLine = ::GetCommandLineW();
	const WIDECHAR* const* ArgV = ::CommandLineToArgvW(CmdLine, &ArgC);
	FMalloc* OutMalloc = MemoryTrace_CreateInternal(InMalloc, ArgC, ArgV);
	::LocalFree(HLOCAL(ArgV));

	if (OutMalloc != InMalloc)
	{
#if defined(PLATFORM_SUPPORTS_TRACE_WIN32_VIRTUAL_MEMORY_HOOKS)
		FVirtualWinApiHooks::Initialize(false);
#endif
	}

	return OutMalloc;
}

#include "Windows/HideWindowsPlatformTypes.h"

#endif // UE_MEMORY_TRACE_ENABLED
