// Copyright Epic Games, Inc. All Rights Reserved.

#if defined(PLATFORM_SUPPORTS_TRACE_WIN32_MODULE_DIAGNOSTICS)

#include "CoreMinimal.h"
#include "Trace/Trace.h"
#include "Trace/Trace.inl"
#include "ProfilingDebugging/ModuleDiagnostics.h"
#include "ProfilingDebugging/MemoryTrace.h"
#include "ProfilingDebugging/MetadataTrace.h"
#include "ProfilingDebugging/TagTrace.h"
#include "HAL/LowLevelMemTracker.h"

#include COMPILED_PLATFORM_HEADER(PlatformModuleDiagnostics.h)

////////////////////////////////////////////////////////////////////////////////
struct FNtDllFunction
{
	FARPROC Addr;

	FNtDllFunction(const char* Name)
	{
		HMODULE NtDll = LoadLibraryW(L"ntdll.dll");
		check(NtDll);
		Addr = GetProcAddress(NtDll, Name);
	}

	template <typename... ArgTypes>
	unsigned int operator () (ArgTypes... Args)
	{
		typedef unsigned int (NTAPI *Prototype)(ArgTypes...);
		return (Prototype((void*)Addr))(Args...);
	}
};

////////////////////////////////////////////////////////////////////////////////
class FModuleTrace
{
public:
	typedef void				(*SubscribeFunc)(bool, void*, const TCHAR*);
	typedef TArray<SubscribeFunc, TFixedAllocator<64>> SubscriberArray;

								FModuleTrace(FMalloc* InMalloc);
								~FModuleTrace();
	static FModuleTrace*		Get();
	void						Initialize();
	void						Subscribe(SubscribeFunc Function);

private:
	void						OnDllLoaded(const UNICODE_STRING& Name, UPTRINT Base);
	void						OnDllUnloaded(UPTRINT Base);
	void						OnDllNotification(unsigned int Reason, const void* DataPtr);
	static FModuleTrace*		Instance;
	SubscriberArray				Subscribers;
	void*						CallbackCookie = nullptr;
	HeapId						ProgramHeapId = 0;
};

////////////////////////////////////////////////////////////////////////////////
FModuleTrace* FModuleTrace::Instance = nullptr;

////////////////////////////////////////////////////////////////////////////////
FModuleTrace::FModuleTrace(FMalloc* InMalloc)
{
	Instance = this;
}

////////////////////////////////////////////////////////////////////////////////
FModuleTrace::~FModuleTrace()
{
	if (CallbackCookie)
	{
		FNtDllFunction UnregisterFunc("LdrUnregisterDllNotification");
		UnregisterFunc(CallbackCookie);
	}
}

////////////////////////////////////////////////////////////////////////////////
FModuleTrace* FModuleTrace::Get()
{
	return Instance;
}

////////////////////////////////////////////////////////////////////////////////
void FModuleTrace::Initialize()
{
	using namespace UE::Trace;

	ProgramHeapId = MemoryTrace_HeapSpec(SystemMemory, TEXT("Module"), EMemoryTraceHeapFlags::None);

	UE_TRACE_LOG(Diagnostics, ModuleInit, ModuleChannel, sizeof(char) * 3)
		<< ModuleInit.SymbolFormat("pdb", 3)
		<< ModuleInit.ModuleBaseShift(uint8(0));

	// Register for DLL load/unload notifications.
	auto Thunk = [] (ULONG Reason, const void* Data, void* Context)
	{
		auto* Self = (FModuleTrace*)Context;
		Self->OnDllNotification(Reason, Data);
	};

	typedef void (CALLBACK *ThunkType)(ULONG, const void*, void*);
	auto ThunkImpl = ThunkType(Thunk);

	FNtDllFunction RegisterFunc("LdrRegisterDllNotification");
	RegisterFunc(0, ThunkImpl, this, &CallbackCookie);

	// Enumerate already loaded modules.
	const TEB* ThreadEnvBlock = NtCurrentTeb();
	const PEB* ProcessEnvBlock = ThreadEnvBlock->ProcessEnvironmentBlock;
	const LIST_ENTRY* ModuleIter = ProcessEnvBlock->Ldr->InMemoryOrderModuleList.Flink;
	const LIST_ENTRY* ModuleIterEnd = ModuleIter->Blink;
	do
	{
		const auto& ModuleData = *(LDR_DATA_TABLE_ENTRY*)(ModuleIter - 1);
		if (ModuleData.DllBase == 0)
		{
			break;
		}

		OnDllLoaded(ModuleData.FullDllName, UPTRINT(ModuleData.DllBase));
		ModuleIter = ModuleIter->Flink;
	}
	while (ModuleIter != ModuleIterEnd);
}

////////////////////////////////////////////////////////////////////////////////
void FModuleTrace::Subscribe(SubscribeFunc Function)
{
	Subscribers.Add(Function);
}

////////////////////////////////////////////////////////////////////////////////
void FModuleTrace::OnDllNotification(unsigned int Reason, const void* DataPtr)
{
	enum
	{
		LDR_DLL_NOTIFICATION_REASON_LOADED = 1,
		LDR_DLL_NOTIFICATION_REASON_UNLOADED = 2,
	};

	struct FNotificationData
	{
		uint32					Flags;
		const UNICODE_STRING&	FullPath;
		const UNICODE_STRING&	BaseName;
		UPTRINT					Base;
	};
	const auto& Data = *(FNotificationData*)DataPtr;

	switch (Reason)
	{
	case LDR_DLL_NOTIFICATION_REASON_LOADED:	OnDllLoaded(Data.FullPath, Data.Base);	break;
	case LDR_DLL_NOTIFICATION_REASON_UNLOADED:	OnDllUnloaded(Data.Base);				break;
	}
}

////////////////////////////////////////////////////////////////////////////////
void FModuleTrace::OnDllLoaded(const UNICODE_STRING& Name, UPTRINT Base)
{
	const auto* DosHeader = (IMAGE_DOS_HEADER*)Base;
	const auto* NtHeaders = (IMAGE_NT_HEADERS*)(Base + DosHeader->e_lfanew);
	const IMAGE_OPTIONAL_HEADER& OptionalHeader = NtHeaders->OptionalHeader;
	TArray<uint8, TFixedAllocator<20>> ImageId;

	// Find the guid and age of the binary, used to match debug files
	const IMAGE_DATA_DIRECTORY& DebugInfoEntry = OptionalHeader.DataDirectory[IMAGE_DIRECTORY_ENTRY_DEBUG];
	const auto* DebugEntries = (IMAGE_DEBUG_DIRECTORY*)(Base + DebugInfoEntry.VirtualAddress);
	for (uint32 i = 0, n = DebugInfoEntry.Size / sizeof(DebugEntries[0]); i < n; ++i)
	{
		const IMAGE_DEBUG_DIRECTORY& Entry = DebugEntries[i];
		if (Entry.Type == IMAGE_DEBUG_TYPE_CODEVIEW)
		{
			struct FCodeView7
			{
				uint32  Signature;
				uint32  Guid[4];
				uint32  Age;
			};

			if (Entry.SizeOfData < sizeof(FCodeView7))
			{
				continue;
			}

			const auto* CodeView7 = (FCodeView7*)(Base + Entry.AddressOfRawData);
			if (CodeView7->Signature != 'SDSR')
			{
				continue;
			}

			ImageId.Append((uint8*)&CodeView7->Guid, sizeof(uint32) * 4);
			ImageId.Append((uint8*)&CodeView7->Age, sizeof(uint32));
			break;
		}
	}

	// Note: UNICODE_STRING.Length is the size in bytes of the string buffer.
	UE_TRACE_LOG(Diagnostics, ModuleLoad, ModuleChannel, Name.Length + ImageId.Num())
		<< ModuleLoad.Name(Name.Buffer, Name.Length / 2)
		<< ModuleLoad.Base(uint64(Base))
		<< ModuleLoad.Size(OptionalHeader.SizeOfImage)
		<< ModuleLoad.ImageId(ImageId.GetData(), ImageId.Num());

#if UE_MEMORY_TRACE_ENABLED
	{
		UE_TRACE_METADATA_CLEAR_SCOPE();
		LLM(UE_MEMSCOPE(ELLMTag::ProgramSize));
		MemoryTrace_Alloc(Base, OptionalHeader.SizeOfImage, 4 * 1024, EMemoryTraceRootHeap::SystemMemory);
		MemoryTrace_MarkAllocAsHeap(Base, ProgramHeapId);
		MemoryTrace_Alloc(Base, OptionalHeader.SizeOfImage, 4 * 1024, EMemoryTraceRootHeap::SystemMemory);
	}
#endif // UE_MEMORY_TRACE_ENABLED

	for (SubscribeFunc Subscriber : Subscribers)
	{
		Subscriber(true, (void*)Base, Name.Buffer);
	}
}

////////////////////////////////////////////////////////////////////////////////
void FModuleTrace::OnDllUnloaded(UPTRINT Base)
{
#if UE_MEMORY_TRACE_ENABLED
	MemoryTrace_Free(Base, EMemoryTraceRootHeap::SystemMemory);
	MemoryTrace_UnmarkAllocAsHeap(Base, ProgramHeapId);
	MemoryTrace_Free(Base, EMemoryTraceRootHeap::SystemMemory);
#endif // UE_MEMORY_TRACE_ENABLED

	UE_TRACE_LOG(Diagnostics, ModuleUnload, ModuleChannel)
		<< ModuleUnload.Base(uint64(Base));

	for (SubscribeFunc Subscriber : Subscribers)
	{
		Subscriber(false, (void*)Base, nullptr);
	}
}



////////////////////////////////////////////////////////////////////////////////
void Modules_Create(FMalloc* Malloc)
{
	if (FModuleTrace::Get() != nullptr)
	{
		return;
	}

	static FModuleTrace Instance(Malloc);
}

////////////////////////////////////////////////////////////////////////////////
void Modules_Initialize()
{
	if (FModuleTrace* Instance = FModuleTrace::Get())
	{
		Instance->Initialize();
	}
}

////////////////////////////////////////////////////////////////////////////////
void Modules_Subscribe(void (*Function)(bool, void*, const TCHAR*))
{
	if (FModuleTrace* Instance = FModuleTrace::Get())
	{
		Instance->Subscribe(Function);
	}
}

#endif // PLATFORM_SUPPORTS_WIN32_MEMORY_TRACE
