// Copyright Epic Games, Inc. All Rights Reserved.
#include "ProfilingDebugging/MiscTrace.h"

#include "Trace/Trace.inl"
#include "Misc/CString.h"
#include "HAL/PlatformTLS.h"
#include "HAL/PlatformTime.h"

UE_TRACE_MINIMAL_CHANNEL(FrameChannel, "Traces the frame durations for different frame types. They can be viewed in the Timing Insights profiler.")
UE_TRACE_CHANNEL_DEFINE(BookmarkChannel, "Low frequency markers signifying important transitions, providing a quick overview for users. Such as level loading or engine boot phases.")
UE_TRACE_CHANNEL(RegionChannel, "Allows tracing of regions. Regions are thread-agnostic timespans in the trace. They can be visualized in the Timing Insights profiler.")
UE_TRACE_CHANNEL(ScreenshotChannel, "Allows screenshots to be included in the trace. Screenshots can be visualized in the Timing Insights profiler.")

UE_TRACE_EVENT_BEGIN(Misc, BookmarkSpec, NoSync|Important)
	UE_TRACE_EVENT_FIELD(const void*, BookmarkPoint)
	UE_TRACE_EVENT_FIELD(int32, Line)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, FormatString)
	UE_TRACE_EVENT_FIELD(UE::Trace::AnsiString, FileName)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Misc, Bookmark)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(const void*, BookmarkPoint)
	UE_TRACE_EVENT_FIELD(uint8[], FormatArgs)
	UE_TRACE_EVENT_FIELD(uint32, CallstackId)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Misc, RegionBegin)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, RegionName)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, Category)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Misc, RegionBeginWithId)
	UE_TRACE_EVENT_FIELD(uint64, CycleAndId)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, RegionName)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, Category)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Misc, RegionEnd)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, RegionName)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Misc, RegionEndWithId)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(uint64, RegionId)
UE_TRACE_EVENT_END()

UE_TRACE_MINIMAL_EVENT_BEGIN(Misc, BeginFrame)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint8, FrameType)
UE_TRACE_MINIMAL_EVENT_END()

UE_TRACE_MINIMAL_EVENT_BEGIN(Misc, EndFrame)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint8, FrameType)
UE_TRACE_MINIMAL_EVENT_END()

UE_TRACE_EVENT_BEGIN(Misc, ScreenshotHeader)
	UE_TRACE_EVENT_FIELD(uint32, Id)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, Name)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(uint32, Width)
	UE_TRACE_EVENT_FIELD(uint32, Height)
	UE_TRACE_EVENT_FIELD(uint32, TotalChunkNum)
	UE_TRACE_EVENT_FIELD(uint32, Size)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Misc, ScreenshotChunk)
	UE_TRACE_EVENT_FIELD(uint32, Id)
	UE_TRACE_EVENT_FIELD(uint32, ChunkNum)
	UE_TRACE_EVENT_FIELD(uint16, Size)
	UE_TRACE_EVENT_FIELD(uint8[], Data)
UE_TRACE_EVENT_END()

void FMiscTrace::OutputBookmarkSpec(const void* BookmarkPoint, const ANSICHAR* File, int32 Line, const TCHAR* Format)
{
	uint16 FileNameLen = uint16(strlen(File));
	uint16 FormatStringLen = uint16(FCString::Strlen(Format));

	uint32 DataSize = (FileNameLen * sizeof(ANSICHAR)) + (FormatStringLen * sizeof(TCHAR));
	UE_TRACE_LOG(Misc, BookmarkSpec, BookmarkChannel, DataSize)
		<< BookmarkSpec.BookmarkPoint(BookmarkPoint)
		<< BookmarkSpec.Line(Line)
		<< BookmarkSpec.FormatString(Format, FormatStringLen)
		<< BookmarkSpec.FileName(File, FileNameLen);
}

void FMiscTrace::OutputBeginRegion(const TCHAR* RegionName, const TCHAR* Category)
{
	UE_TRACE_LOG(Misc, RegionBegin, RegionChannel)
		<< RegionBegin.Cycle(FPlatformTime::Cycles64())
		<< RegionBegin.RegionName(RegionName)
		<< RegionBegin.Category(Category == nullptr ? TEXT("") : Category);
}

uint64 FMiscTrace::OutputBeginRegionWithId(const TCHAR* RegionName, const TCHAR* Category)
{
	const uint64 CycleAndId = FPlatformTime::Cycles64();
	UE_TRACE_LOG(Misc, RegionBeginWithId, RegionChannel)
		<< RegionBeginWithId.CycleAndId(CycleAndId)
		<< RegionBeginWithId.RegionName(RegionName)
		<< RegionBeginWithId.Category(Category == nullptr ? TEXT("") : Category);
	return CycleAndId;
}

void FMiscTrace::OutputEndRegion(const TCHAR* RegionName)
{
	UE_TRACE_LOG(Misc, RegionEnd, RegionChannel)
		<< RegionEnd.Cycle(FPlatformTime::Cycles64())
		<< RegionEnd.RegionName(RegionName);
}

void FMiscTrace::OutputEndRegionWithId(uint64 RegionId)
{
	UE_TRACE_LOG(Misc, RegionEndWithId, RegionChannel)
		<< RegionEndWithId.Cycle(FPlatformTime::Cycles64())
		<< RegionEndWithId.RegionId(RegionId);
}

void FMiscTrace::OutputBookmarkInternal(const void* BookmarkPoint, uint32 CallstackId, uint16 EncodedFormatArgsSize, uint8* EncodedFormatArgs)
{
	OutputBookmarkInternalCycles(FPlatformTime::Cycles64(), BookmarkPoint, CallstackId, EncodedFormatArgsSize, EncodedFormatArgs);
}

void FMiscTrace::OutputBookmarkInternalCycles(uint64 Cycles, const void* BookmarkPoint, uint32 CallstackId, uint16 EncodedFormatArgsSize, uint8* EncodedFormatArgs)
{
	UE_TRACE_LOG(Misc, Bookmark, BookmarkChannel)
		<< Bookmark.Cycle(Cycles)
		<< Bookmark.BookmarkPoint(BookmarkPoint)
		<< Bookmark.FormatArgs(EncodedFormatArgs, EncodedFormatArgsSize)
		<< Bookmark.CallstackId(CallstackId);
}

void FMiscTrace::OutputBeginFrame(ETraceFrameType FrameType)
{
	if (!UE_TRACE_MINIMAL_CHANNELEXPR_IS_ENABLED(FrameChannel))
	{
		return;
	}

	uint64 Cycle = FPlatformTime::Cycles64();
	UE_TRACE_MINIMAL_LOG(Misc, BeginFrame, FrameChannel)
		<< BeginFrame.Cycle(Cycle)
		<< BeginFrame.FrameType((uint8)FrameType);
}

void FMiscTrace::OutputEndFrame(ETraceFrameType FrameType)
{
	if (!UE_TRACE_MINIMAL_CHANNELEXPR_IS_ENABLED(FrameChannel))
	{
		return;
	}

	uint64 Cycle = FPlatformTime::Cycles64();
	UE_TRACE_MINIMAL_LOG(Misc, EndFrame, FrameChannel)
		<< EndFrame.Cycle(Cycle)
		<< EndFrame.FrameType((uint8)FrameType);
}

void FMiscTrace::OutputScreenshot(const TCHAR* Name, uint64 Cycle, uint32 Width, uint32 Height, TArray64<uint8> Data)
{
	static std::atomic<uint32> ScreenshotId = 0;

	const uint32 DataSize = (uint32) Data.Num();
	const uint32 MaxChunkSize = TNumericLimits<uint16>::Max();
	uint32 ChunkNum = (DataSize + MaxChunkSize - 1) / MaxChunkSize;

	uint32 Id = ScreenshotId.fetch_add(1);
	UE_TRACE_LOG(Misc, ScreenshotHeader, ScreenshotChannel)
		<< ScreenshotHeader.Id(Id)
		<< ScreenshotHeader.Name(Name, uint16(FCString::Strlen(Name)))
		<< ScreenshotHeader.Cycle(Cycle)
		<< ScreenshotHeader.Width(Width)
		<< ScreenshotHeader.Height(Height)
		<< ScreenshotHeader.TotalChunkNum(ChunkNum)
		<< ScreenshotHeader.Size(DataSize);

	uint32 RemainingSize = DataSize;
	for (uint32 Index = 0; Index < ChunkNum; ++Index)
	{
		uint16 Size = (uint16) FMath::Min(RemainingSize, MaxChunkSize);

		uint8* ChunkData = Data.GetData() + MaxChunkSize * Index;
		UE_TRACE_LOG(Misc, ScreenshotChunk, ScreenshotChannel)
			<< ScreenshotChunk.Id(Id)
			<< ScreenshotChunk.ChunkNum(Index)
			<< ScreenshotChunk.Size(Size)
			<< ScreenshotChunk.Data(ChunkData, Size);

		RemainingSize -= Size;
	}

	check(RemainingSize == 0);
}

bool FMiscTrace::ShouldTraceScreenshot()
{
	return UE_TRACE_CHANNELEXPR_IS_ENABLED(ScreenshotChannel);
}

bool FMiscTrace::ShouldTraceBookmark()
{
	return UE_TRACE_CHANNELEXPR_IS_ENABLED(BookmarkChannel);
}

bool FMiscTrace::ShouldTraceRegion()
{
	return UE_TRACE_CHANNELEXPR_IS_ENABLED(RegionChannel);
}
