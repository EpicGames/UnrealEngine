// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProfilingDebugging/ModuleDiagnostics.h"

#include "Trace/Detail/Channel.h"

////////////////////////////////////////////////////////////////////////////////
UE_TRACE_CHANNEL_DEFINE(ModuleChannel, "Module information needed for symbols resolution", true)
 
UE_TRACE_EVENT_DEFINE(Diagnostics, ModuleInit)
UE_TRACE_EVENT_DEFINE(Diagnostics, ModuleLoad)
UE_TRACE_EVENT_DEFINE(Diagnostics, ModuleUnload)
