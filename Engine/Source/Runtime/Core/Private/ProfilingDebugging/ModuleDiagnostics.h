// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Trace/Trace.h"
#include "Trace/Trace.inl"

namespace UE { namespace Trace { class FChannel; } }

////////////////////////////////////////////////////////////////////////////////
UE_TRACE_CHANNEL_EXTERN(ModuleChannel)
 
UE_TRACE_EVENT_BEGIN_EXTERN(Diagnostics, ModuleInit, NoSync|Important)
	UE_TRACE_EVENT_FIELD(UE::Trace::AnsiString, SymbolFormat)
	UE_TRACE_EVENT_FIELD(uint8, ModuleBaseShift)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN_EXTERN(Diagnostics, ModuleLoad, NoSync|Important)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, Name)
	UE_TRACE_EVENT_FIELD(uint64, Base)
	UE_TRACE_EVENT_FIELD(uint32, Size)
	UE_TRACE_EVENT_FIELD(uint8[], ImageId) // Platform specific id for this image, used to match debug files were available
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN_EXTERN(Diagnostics, ModuleUnload, NoSync|Important)
	UE_TRACE_EVENT_FIELD(uint64, Base)
UE_TRACE_EVENT_END()
