// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProfilingDebugging/PlatformEvents.h"
#include "Containers/UnrealString.h"

/////////////////////////////////////////////////////////////////////

#if UE_TRACE_ENABLED
class FCSChannel : public UE::Trace::FChannel
{
	virtual bool OnToggle(bool bNewState, const TCHAR** OutReason) override
	{
#if PLATFORM_SUPPORTS_PLATFORM_EVENTS
		return FPlatformEventsTrace::CanEnable(FPlatformEventsTrace::EEventType::ContextSwitch, OutReason);
#else
		return false;
#endif
	}
	virtual void OnToggled(bool bNewState) override
	{
#if PLATFORM_SUPPORTS_PLATFORM_EVENTS
		if (bNewState)
		{
			FPlatformEventsTrace::Enable(FPlatformEventsTrace::EEventType::ContextSwitch);
		}
		else
		{
			FPlatformEventsTrace::Disable(FPlatformEventsTrace::EEventType::ContextSwitch);
		}
#endif
	}
};

class FSSChannel : public UE::Trace::FChannel
{
	virtual bool OnToggle(bool bNewState, const TCHAR** OutReason) override
	{
#if PLATFORM_SUPPORTS_PLATFORM_EVENTS
		return FPlatformEventsTrace::CanEnable(FPlatformEventsTrace::EEventType::StackSampling, OutReason);
#else 
		return false;
#endif
	}
	virtual void OnToggled(bool bNewState) override
	{
#if PLATFORM_SUPPORTS_PLATFORM_EVENTS
		if (bNewState)
		{
			FPlatformEventsTrace::Enable(FPlatformEventsTrace::EEventType::StackSampling);
		}
		else
		{
			FPlatformEventsTrace::Disable(FPlatformEventsTrace::EEventType::StackSampling);
		}
#endif
	}
};
#endif

UE_TRACE_CHANNEL_CUSTOM_DEFINE(ContextSwitchChannel, FCSChannel, "Trace CPU context switches. On Windows, game/editor runtime should be run as administrator", false)
UE_TRACE_CHANNEL_CUSTOM_DEFINE(StackSamplingChannel, FSSChannel, "Trace stack sampling events. On Windows, game/editor runtime should be run as administrator", false)

/////////////////////////////////////////////////////////////////////

// Represents the settings for Context Switches and Stack Sampling
UE_TRACE_EVENT_BEGIN(PlatformEvent, Settings, NoSync | Important)
	UE_TRACE_EVENT_FIELD(uint32, SamplingInterval) // [microseconds]
UE_TRACE_EVENT_END()

// Represents time interval when thread was running on specific core
UE_TRACE_EVENT_BEGIN(PlatformEvent, ContextSwitch, NoSync)
	UE_TRACE_EVENT_FIELD(uint64, StartTime)
	UE_TRACE_EVENT_FIELD(uint64, EndTime)
	UE_TRACE_EVENT_FIELD(uint32, ThreadId) // system thread id
	UE_TRACE_EVENT_FIELD(uint8, CoreNumber)
UE_TRACE_EVENT_END()

// Represents call stack addresses in Stack Sampling
UE_TRACE_EVENT_BEGIN(PlatformEvent, StackSample, NoSync)
	UE_TRACE_EVENT_FIELD(uint64, Time)
	UE_TRACE_EVENT_FIELD(uint32, ThreadId) // system thread id
	UE_TRACE_EVENT_FIELD(uint64[], Addresses)
UE_TRACE_EVENT_END()

// Thread information for Context Switches and Stack Sampling
// NOTE: In some cases, name of process can be empty, for example when there are no
// privileges to query it or there is no name for the process thread belongs to.
// Depending on platform, actual name can be absolute path to process executable.
UE_TRACE_EVENT_BEGIN(PlatformEvent, ThreadName, NoSync)
	UE_TRACE_EVENT_FIELD(uint32, ThreadId) // system thread id
	UE_TRACE_EVENT_FIELD(uint32, ProcessId)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, Name)
UE_TRACE_EVENT_END()

/////////////////////////////////////////////////////////////////////

#if PLATFORM_SUPPORTS_PLATFORM_EVENTS

void FPlatformEventsTrace::OutputSettings(uint32 SamplingIntervalUsec)
{
	bool bShouldTrace = UE_TRACE_CHANNELEXPR_IS_ENABLED(StackSamplingChannel) || UE_TRACE_CHANNELEXPR_IS_ENABLED(ContextSwitchChannel); //-V501
	UE_TRACE_LOG(PlatformEvent, Settings, bShouldTrace)
		<< Settings.SamplingInterval(SamplingIntervalUsec);
}

/////////////////////////////////////////////////////////////////////

void FPlatformEventsTrace::OutputContextSwitch(uint64 StartTime, uint64 EndTime, uint32 ThreadId, uint8 CoreNumber)
{
	UE_TRACE_LOG(PlatformEvent, ContextSwitch, ContextSwitchChannel)
		<< ContextSwitch.StartTime(StartTime)
		<< ContextSwitch.EndTime(EndTime)
		<< ContextSwitch.ThreadId(ThreadId)
		<< ContextSwitch.CoreNumber(CoreNumber);
}

/////////////////////////////////////////////////////////////////////

void FPlatformEventsTrace::OutputStackSample(uint64 Time, uint32 ThreadId, const uint64* Addresses, uint32 AddressCount)
{
	UE_TRACE_LOG(PlatformEvent, StackSample, StackSamplingChannel)
		<< StackSample.Time(Time)
		<< StackSample.ThreadId(ThreadId)
		<< StackSample.Addresses(Addresses, AddressCount);
}

/////////////////////////////////////////////////////////////////////

void FPlatformEventsTrace::OutputThreadName(uint32 ThreadId, uint32 ProcessId, const TCHAR* Name, uint32 NameLen)
{
	bool bShouldTrace = UE_TRACE_CHANNELEXPR_IS_ENABLED(StackSamplingChannel) || UE_TRACE_CHANNELEXPR_IS_ENABLED(ContextSwitchChannel); //-V501
	UE_TRACE_LOG(PlatformEvent, ThreadName, bShouldTrace)
		<< ThreadName.ThreadId(ThreadId)
		<< ThreadName.ProcessId(ProcessId)
		<< ThreadName.Name(Name, NameLen);
}

/////////////////////////////////////////////////////////////////////

FPlatformEventsTrace::EEventType FPlatformEventsTrace::GetEvent(const FString& Name)
{
	if (Name == TEXT("contextswitch"))
	{
		return FPlatformEventsTrace::EEventType::ContextSwitch;
	}
	else if (Name == TEXT("stacksampling"))
	{
		return FPlatformEventsTrace::EEventType::StackSampling;
	}
	else
	{
		return FPlatformEventsTrace::EEventType::None;
	}
}

/////////////////////////////////////////////////////////////////////

void FPlatformEventsTrace::OnTraceChannelUpdated(const FString& ChannelName, bool bIsEnabled)
{
	FPlatformEventsTrace::EEventType Event = FPlatformEventsTrace::GetEvent(ChannelName.Replace(TEXT("Channel"), TEXT("")));

	if (Event != FPlatformEventsTrace::EEventType::None)
	{
		if (bIsEnabled)
		{
			FPlatformEventsTrace::Enable(Event);
		}
		else
		{
			FPlatformEventsTrace::Disable(Event);
		}
	}
}

/////////////////////////////////////////////////////////////////////

void FPlatformEventsTrace::PostInit()
{
	if (UE_TRACE_CHANNELEXPR_IS_ENABLED(ContextSwitchChannel)) //-V517
	{
		Enable(FPlatformEventsTrace::EEventType::ContextSwitch);
	}

	if (UE_TRACE_CHANNELEXPR_IS_ENABLED(StackSamplingChannel))
	{
		Enable(FPlatformEventsTrace::EEventType::StackSampling);
	}
}

#endif // PLATFORM_SUPPORTS_PLATFORM_EVENTS

/////////////////////////////////////////////////////////////////////
