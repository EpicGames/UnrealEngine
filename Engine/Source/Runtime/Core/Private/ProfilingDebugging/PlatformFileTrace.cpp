// Copyright Epic Games, Inc. All Rights Reserved.
#include "ProfilingDebugging/PlatformFileTrace.h"

#if PLATFORMFILETRACE_ENABLED

#include "Containers/Map.h"
#include "CoreGlobals.h"
#include "HAL/CriticalSection.h"
#include "HAL/PlatformTime.h"
#include "HAL/PlatformTLS.h"
#include "Misc/CString.h"
#include "Misc/Parse.h"
#include "Misc/ScopeLock.h"
#include "Templates/Atomic.h"
#include "Trace/Trace.inl"

#define PLATFORMFILETRACE_DEBUG_ENABLED 0

UE_TRACE_CHANNEL(FileChannel, "Platform file I/O operations: open, close, read, and write events with file handle, offset, size, and cycle timestamps.")

UE_TRACE_EVENT_BEGIN(PlatformFile, BeginOpen)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, Path)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(PlatformFile, EndOpen)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(uint64, FileHandle)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(PlatformFile, BeginReOpen)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(uint64, OldFileHandle)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(PlatformFile, EndReOpen)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(uint64, NewFileHandle)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(PlatformFile, BeginClose)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(uint64, FileHandle)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(PlatformFile, EndClose)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(PlatformFile, BeginRead)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(uint64, ReadHandle)
	UE_TRACE_EVENT_FIELD(uint64, FileHandle)
	UE_TRACE_EVENT_FIELD(uint64, Offset)
	UE_TRACE_EVENT_FIELD(uint64, Size)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(PlatformFile, EndRead)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(uint64, ReadHandle)
	UE_TRACE_EVENT_FIELD(uint64, SizeRead)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(PlatformFile, BeginWrite)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(uint64, WriteHandle)
	UE_TRACE_EVENT_FIELD(uint64, FileHandle)
	UE_TRACE_EVENT_FIELD(uint64, Offset)
	UE_TRACE_EVENT_FIELD(uint64, Size)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(PlatformFile, EndWrite)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(uint64, WriteHandle)
	UE_TRACE_EVENT_FIELD(uint64, SizeWritten)
UE_TRACE_EVENT_END()

namespace
{
#if PLATFORMFILETRACE_DEBUG_ENABLED
	FCriticalSection OpenHandlesLock;
	TMap<uint64, int32> OpenHandles;
#else
	TAtomic<int32> OpenFileHandleCount;
#endif
}

void FPlatformFileTrace::BeginOpen(const TCHAR* Path)
{
	int32 PathLength = FCString::Strlen(Path);
	UE_TRACE_LOG(PlatformFile, BeginOpen, FileChannel)
		<< BeginOpen.Cycle(FPlatformTime::Cycles64())
		<< BeginOpen.Path(Path, PathLength);
}

void FPlatformFileTrace::EndOpen(uint64 FileHandle)
{
	UE_TRACE_LOG(PlatformFile, EndOpen, FileChannel)
		<< EndOpen.Cycle(FPlatformTime::Cycles64())
		<< EndOpen.FileHandle(FileHandle);
#if PLATFORMFILETRACE_DEBUG_ENABLED
	{
		FScopeLock OpenHandlesScopeLock(&OpenHandlesLock);
		++OpenHandles.FindOrAdd(FileHandle, 0);
	}
#else
	++OpenFileHandleCount;
#endif
}

void FPlatformFileTrace::FailOpen(const TCHAR* Path)
{
	UE_TRACE_LOG(PlatformFile, EndOpen, FileChannel)
		<< EndOpen.Cycle(FPlatformTime::Cycles64())
		<< EndOpen.FileHandle(uint64(-1));
}

void FPlatformFileTrace::BeginReOpen(uint64 OldFileHandle)
{
	UE_TRACE_LOG(PlatformFile, BeginReOpen, FileChannel)
		<< BeginReOpen.Cycle(FPlatformTime::Cycles64())
		<< BeginReOpen.OldFileHandle(OldFileHandle);
}

void FPlatformFileTrace::EndReOpen(uint64 NewFileHandle)
{
	UE_TRACE_LOG(PlatformFile, EndReOpen, FileChannel)
		<< EndReOpen.Cycle(FPlatformTime::Cycles64())
		<< EndReOpen.NewFileHandle(NewFileHandle);
#if PLATFORMFILETRACE_DEBUG_ENABLED
	{
		FScopeLock OpenHandlesScopeLock(&OpenHandlesLock);
		++OpenHandles.FindOrAdd(NewFileHandle, 0);
	}
#else
	++OpenFileHandleCount;
#endif
}

void FPlatformFileTrace::BeginClose(uint64 FileHandle)
{
	UE_TRACE_LOG(PlatformFile, BeginClose, FileChannel)
		<< BeginClose.Cycle(FPlatformTime::Cycles64())
		<< BeginClose.FileHandle(FileHandle);
}

void FPlatformFileTrace::EndClose(uint64 FileHandle)
{
	UE_TRACE_LOG(PlatformFile, EndClose, FileChannel)
		<< EndClose.Cycle(FPlatformTime::Cycles64());
#if PLATFORMFILETRACE_DEBUG_ENABLED
	bool bUnderflow = false;
	{
		FScopeLock OpenHandlesScopeLock(&OpenHandlesLock);
		int32& OpenCount = OpenHandles.FindOrAdd(FileHandle, 0);
		--OpenCount;
		if (OpenCount <= 0)
		{
			bUnderflow = OpenCount < 0;
			OpenHandles.Remove(FileHandle);
		}
	}
	if (bUnderflow)
	{
		UE_LOGF(LogCore, Error, "FPlatformFileTrace Close without an Open: FileHandle %llu.", FileHandle);
	}
#else
	int32 NewValue = --OpenFileHandleCount;
	if (NewValue == -1)
	{
		UE_LOGF(LogCore, Error, "FPlatformFileTrace Close without an Open");
		++OpenFileHandleCount; // clamp the value to 0
	}
#endif
}

void FPlatformFileTrace::FailClose(uint64 FileHandle)
{
	UE_TRACE_LOG(PlatformFile, EndClose, FileChannel)
		<< EndClose.Cycle(FPlatformTime::Cycles64());
}

void FPlatformFileTrace::BeginRead(uint64 ReadHandle, uint64 FileHandle, uint64 Offset, uint64 Size)
{
	UE_TRACE_LOG(PlatformFile, BeginRead, FileChannel)
		<< BeginRead.Cycle(FPlatformTime::Cycles64())
		<< BeginRead.ReadHandle(ReadHandle)
		<< BeginRead.FileHandle(FileHandle)
		<< BeginRead.Offset(Offset)
		<< BeginRead.Size(Size);
}

void FPlatformFileTrace::EndRead(uint64 ReadHandle, uint64 SizeRead)
{
	UE_TRACE_LOG(PlatformFile, EndRead, FileChannel)
		<< EndRead.Cycle(FPlatformTime::Cycles64())
		<< EndRead.ReadHandle(ReadHandle)
		<< EndRead.SizeRead(SizeRead);
}

void FPlatformFileTrace::BeginWrite(uint64 WriteHandle, uint64 FileHandle, uint64 Offset, uint64 Size)
{
	UE_TRACE_LOG(PlatformFile, BeginWrite, FileChannel)
		<< BeginWrite.Cycle(FPlatformTime::Cycles64())
		<< BeginWrite.WriteHandle(WriteHandle)
		<< BeginWrite.FileHandle(FileHandle)
		<< BeginWrite.Offset(Offset)
		<< BeginWrite.Size(Size);
}

void FPlatformFileTrace::EndWrite(uint64 WriteHandle, uint64 SizeWritten)
{
	UE_TRACE_LOG(PlatformFile, EndWrite, FileChannel)
		<< EndWrite.Cycle(FPlatformTime::Cycles64())
		<< EndWrite.WriteHandle(WriteHandle)
		<< EndWrite.SizeWritten(SizeWritten);
}

uint32 FPlatformFileTrace::GetOpenFileHandleCount()
{
#if PLATFORMFILETRACE_DEBUG_ENABLED
	return OpenHandles.Num();
#else
	return OpenFileHandleCount;
#endif
}

#endif
