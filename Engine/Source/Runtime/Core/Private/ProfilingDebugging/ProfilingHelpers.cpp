// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Here are a number of profiling helper functions so we do not have to duplicate a lot of the glue
 * code everywhere.  And we can have consistent naming for all our files.
 *
 */

// Core includes.
#include "ProfilingDebugging/ProfilingHelpers.h"
#include "Misc/DateTime.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformProcess.h"
#include "Misc/Parse.h"
#include "Misc/CommandLine.h"
#include "Misc/Paths.h"
#include "Misc/EngineVersion.h"

// find where these are really defined
const static int32 MaxFilenameLen = 100;

#if WITH_ENGINE
FGetMapNameDelegate GGetMapNameDelegate;
#endif

/**
 * This will get the changelist that should be used with the Automated Performance Testing
 * If one is passed in we use that otherwise we use the FEngineVersion::Current().GetChangelist().  This allows
 * us to have build machine built .exe and test them. 
 *
 * NOTE: had to use AutomatedBenchmarking as the parsing code is flawed and doesn't match
 *       on whole words.  so automatedperftestingChangelist was failing :-( 
 **/
int32 GetChangeListNumberForPerfTesting()
{
	int32 Retval = FEngineVersion::Current().GetChangelist();

	// if we have passed in the changelist to use then use it
	int32 FromCommandLine = 0;
	FParse::Value( FCommandLine::Get(), TEXT("-gABC="), FromCommandLine );

	// we check for 0 here as the CIS always appends -AutomatedPerfChangelist but if it
	// is from the "built" builds when it will be a 0
	if( FromCommandLine != 0 )
	{
		Retval = FromCommandLine;
	}

	return Retval;
}

// can be optimized or moved to a more central spot
bool IsValidCPPIdentifier(const FString& In)
{
	bool bFirstChar = true;

	for (auto& Char : In)
	{
		if (!bFirstChar && Char >= TCHAR('0') && Char <= TCHAR('9'))
		{
			// 0..9 is allowed unless on first character
			continue;
		}

		bFirstChar = false;

		if (Char == TCHAR('_')
			|| (Char >= TCHAR('a') && Char <= TCHAR('z'))
			|| (Char >= TCHAR('A') && Char <= TCHAR('Z')))
		{
			continue;
		}

		return false;
	}

	return true;
}

FString GetBuildNameForPerfTesting()
{
	FString BuildName;

	if (FParse::Value(FCommandLine::Get(), TEXT("-BuildName="), BuildName))
	{
		// we take the specified name
		if (!IsValidCPPIdentifier(BuildName))
		{
			UE_LOGF(LogInit, Error, "The name specified by -BuildName=<name> is not valid (needs to follow C++ identifier rules)");
			BuildName.Empty();
		}
	}
	
	if(BuildName.IsEmpty())
	{
		// we generate a build name from the change list number
		int32 ChangeListNumber = GetChangeListNumberForPerfTesting();

		BuildName = FString::Printf(TEXT("CL%d"), ChangeListNumber);
	}

	return BuildName;
}


/**
 * This makes it so UnrealConsole will open up the memory profiler for us
 *
 * @param NotifyType has the <namespace>:<type> (e.g. UE_PROFILER!UE3STATS:)
 * @param FullFileName the File name to copy from the console
 **/
void SendDataToPCViaUnrealConsole( const FString& NotifyType, const FString& FullFileName )
{
	const FString AbsoluteFilename = IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead( *FullFileName );

	UE_LOGF( LogProfilingDebugging, Warning, "SendDataToPCViaUnrealConsole %ls%ls", *NotifyType, *AbsoluteFilename );

	const FString NotifyString = NotifyType + AbsoluteFilename + LINE_TERMINATOR;
	
	// send it across via UnrealConsole
	FMsg::SendNotificationString( *NotifyString );
}


FString CreateProfileFilename(const FString& InFileExtension, bool bIncludeDateForDirectoryName)
{
	return CreateProfileFilename(TEXT(""), InFileExtension, bIncludeDateForDirectoryName);
}

/** 
 * This will generate the profiling file name that will work with limited filename sizes on consoles.
 * We want a uniform naming convention so we will all just call this function.
 *
 * @param ProfilingType this is the type of profiling file this is
 * 
 **/
FString CreateProfileFilename( const FString& InFilename, const FString& InFileExtension, bool bIncludeDateForDirectoryName )
{
	FString Retval;

	// set up all of the parts we will use
	FString MapNameStr;

#if WITH_ENGINE
	if(GGetMapNameDelegate.IsBound())
	{
		MapNameStr = GGetMapNameDelegate.Execute();
	}
	else
	{
		MapNameStr = TEXT( "LoadTimeFile" );
	}
#endif		// WITH_ENGINE

	const FString PlatformStr(FPlatformProperties::PlatformName());

	/** This is meant to hold the name of the "sessions" that is occurring **/
	static bool bSetProfilingSessionFolderName = false;
	static FString ProfilingSessionFolderName; 

	// here we want to have just the same profiling session name so all of the files will go into that folder over the course of the run so you don't just have a ton of folders
	FString FolderName;
	if( bSetProfilingSessionFolderName == false )
	{
		// now create the string
		FolderName = FString::Printf(TEXT("%s-%s-%s"), *MapNameStr, *PlatformStr, *FDateTime::Now().ToString(TEXT("%m.%d-%H.%M.%S")));
		FolderName.RightInline(MaxFilenameLen, EAllowShrinking::No);

		ProfilingSessionFolderName = FolderName;
		bSetProfilingSessionFolderName = true;
	}
	else
	{
		FolderName = ProfilingSessionFolderName;
	}

	// now create the string
	// NOTE: due to the changelist this is implicitly using the same directory
	FString FolderNameOfProfileNoDate = FString::Printf( TEXT("%s-%s-%i"), *MapNameStr, *PlatformStr, GetChangeListNumberForPerfTesting() );
	FolderNameOfProfileNoDate.RightInline(MaxFilenameLen, EAllowShrinking::No);


	FString NameOfProfile;
	if (InFilename.IsEmpty())
	{
		NameOfProfile = FString::Printf(TEXT("%s-%s-%s"), *MapNameStr, *PlatformStr, *FDateTime::Now().ToString(TEXT("%d-%H.%M.%S")));
	}
	else
	{
		NameOfProfile = InFilename;
	}
	NameOfProfile.RightInline(MaxFilenameLen, EAllowShrinking::No);

#if WITH_SERVER_CODE
	FString FileNameWithExtension = FString::Printf(TEXT("Pid%d_%s%s"), FPlatformProcess::GetCurrentProcessId(), *NameOfProfile, *InFileExtension);
#else
	FString FileNameWithExtension = FString::Printf(TEXT("%s%s"), *NameOfProfile, *InFileExtension);
#endif

	FileNameWithExtension.RightInline(MaxFilenameLen, EAllowShrinking::No);

	FString Filename;
	if( bIncludeDateForDirectoryName == true )
	{
		Filename = FolderName / FileNameWithExtension;
	}
	else
	{
		Filename = FolderNameOfProfileNoDate / FileNameWithExtension;
	}


	Retval = Filename;

	return Retval;
}



FString CreateProfileDirectoryAndFilename( const FString& InSubDirectoryName, const FString& InFileExtension )
{
	FString MapNameStr;
#if WITH_ENGINE
	check(GGetMapNameDelegate.IsBound());
	MapNameStr = GGetMapNameDelegate.Execute();
#endif		// WITH_ENGINE
	const FString PlatformStr(FPlatformProperties::PlatformName());


	// create Profiling dir and sub dir
	const FString PathName = (FPaths::ProfilingDir() + InSubDirectoryName + TEXT("/"));
	IFileManager::Get().MakeDirectory( *PathName );
	//UE_LOGF(LogProfilingDebugging, Warning, "CreateProfileDirectoryAndFilename: %ls", *PathName );

	// create the directory name of this profile
	FString NameOfProfile = FString::Printf(TEXT("%s-%s-%s"), *MapNameStr, *PlatformStr, *FDateTime::Now().ToString(TEXT("%m.%d-%H.%M")));	
	NameOfProfile.RightInline(MaxFilenameLen, EAllowShrinking::No);

	IFileManager::Get().MakeDirectory( *(PathName+NameOfProfile) );
	//UE_LOGF(LogProfilingDebugging, Warning, "CreateProfileDirectoryAndFilename: %ls", *(PathName+NameOfProfile) );


	// create the actual file name
	FString FileNameWithExtension = FString::Printf( TEXT("%s%s"), *NameOfProfile, *InFileExtension );
	FileNameWithExtension.LeftInline(MaxFilenameLen, EAllowShrinking::No);
	//UE_LOGF(LogProfilingDebugging, Warning, "CreateProfileDirectoryAndFilename: %ls", *FileNameWithExtension );


	FString Filename = PathName / NameOfProfile / FileNameWithExtension;
	//UE_LOGF(LogProfilingDebugging, Warning, "CreateProfileDirectoryAndFilename: %ls", *Filename );

	return Filename;
}
