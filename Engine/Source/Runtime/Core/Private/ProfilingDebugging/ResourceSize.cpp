// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProfilingDebugging/ResourceSize.h"

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Misc/AssertionMacros.h"
#include "Misc/OutputDevice.h"

FResourceSizeEx::FResourceSizeEx()
	: ResourceSizeMode(EResourceSizeMode::Exclusive)
{
}


FResourceSizeEx::FResourceSizeEx(const EResourceSizeMode::Type InResourceSizeMode)
	: ResourceSizeMode(InResourceSizeMode)
{
}

FResourceSizeEx::FResourceSizeEx(const EResourceSizeMode::Type InResourceSizeMode, const SIZE_T InDedicatedSystemMemoryBytes, const SIZE_T InDedicatedVideoMemoryBytes)
	: ResourceSizeMode(InResourceSizeMode)
{
	DedicatedSystemMemoryBytesMap.Add(TEXT("Untracked Memory"), InDedicatedSystemMemoryBytes);
	DedicatedVideoMemoryBytesMap.Add(TEXT("Untracked Memory"), InDedicatedVideoMemoryBytes);
}

FResourceSizeEx::FResourceSizeEx(const EResourceSizeMode::Type InResourceSizeMode, const SIZE_T InUnknownMemoryBytes)
	: ResourceSizeMode(InResourceSizeMode)
{
	UnknownMemoryBytesMap.Add(TEXT("Untracked Memory"), InUnknownMemoryBytes);
}

void FResourceSizeEx::LogSummary(FOutputDevice& Ar) const
{
	auto PrintPair = [&Ar](const TPair<FName, SIZE_T>& Pair)
	{
		Ar.Logf(
			TEXT("%140s %15.2f"),
			*Pair.Key.ToString(),
			static_cast<double>(Pair.Value) / 1024.0
		);
	};

	for (const TPair<FName, SIZE_T>& Pair : DedicatedSystemMemoryBytesMap)
	{
		PrintPair(Pair);
	}
	for (const TPair<FName, SIZE_T>& Pair : DedicatedVideoMemoryBytesMap)
	{
		PrintPair(Pair);
	}
	for (const TPair<FName, SIZE_T>& Pair : UnknownMemoryBytesMap)
	{
		PrintPair(Pair);
	}
}

EResourceSizeMode::Type FResourceSizeEx::GetResourceSizeMode() const
{
	return ResourceSizeMode;
}

FResourceSizeEx& FResourceSizeEx::AddDedicatedSystemMemoryBytes(const FName& Tag, const SIZE_T InMemoryBytes)
{
	SIZE_T& CurrentSize = DedicatedSystemMemoryBytesMap.FindOrAdd(Tag);
	CurrentSize += InMemoryBytes;
	return *this;
}

FResourceSizeEx& FResourceSizeEx::AddDedicatedSystemMemoryBytes(const SIZE_T InMemoryBytes)
{
	SIZE_T& CurrentSize = DedicatedSystemMemoryBytesMap.FindOrAdd(TEXT("Untracked Memory"));
	CurrentSize += InMemoryBytes;
	return *this;
}

SIZE_T FResourceSizeEx::GetDedicatedSystemMemoryBytes() const
{
	SIZE_T Sum = 0;
	TArray<SIZE_T> Values;
	DedicatedSystemMemoryBytesMap.GenerateValueArray(Values);
	for (const SIZE_T s : Values) Sum += s;
	return Sum;
}

FResourceSizeEx& FResourceSizeEx::AddDedicatedVideoMemoryBytes(const FName& Tag, const SIZE_T InMemoryBytes)
{
	SIZE_T& CurrentSize = DedicatedVideoMemoryBytesMap.FindOrAdd(Tag);
	CurrentSize += InMemoryBytes;
	return *this;
}

FResourceSizeEx& FResourceSizeEx::AddDedicatedVideoMemoryBytes(const SIZE_T InMemoryBytes)
{
	SIZE_T& CurrentSize = DedicatedVideoMemoryBytesMap.FindOrAdd(TEXT("Untracked Memory"));
	CurrentSize += InMemoryBytes;
	return *this;
}

SIZE_T FResourceSizeEx::GetDedicatedVideoMemoryBytes() const
{
	SIZE_T Sum = 0;
	TArray<SIZE_T> Values;
	DedicatedVideoMemoryBytesMap.GenerateValueArray(Values);
	for (const SIZE_T s : Values) Sum += s;
	return Sum;
}

FResourceSizeEx& FResourceSizeEx::AddUnknownMemoryBytes(const FName& Tag, const SIZE_T InMemoryBytes)
{
	SIZE_T& CurrentSize = UnknownMemoryBytesMap.FindOrAdd(Tag);
	CurrentSize += InMemoryBytes;
	return *this;
}

FResourceSizeEx& FResourceSizeEx::AddUnknownMemoryBytes(const SIZE_T InMemoryBytes)
{
	SIZE_T& CurrentSize = UnknownMemoryBytesMap.FindOrAdd(TEXT("Untracked Memory"));
	CurrentSize += InMemoryBytes;
	return *this;
}

SIZE_T FResourceSizeEx::GetUnknownMemoryBytes() const
{
	SIZE_T Sum = 0;
	TArray<SIZE_T> Values;
	UnknownMemoryBytesMap.GenerateValueArray(Values);
	for (const SIZE_T s : Values) Sum += s;
	return Sum;
}

SIZE_T FResourceSizeEx::GetTotalMemoryBytes() const
{
	return GetDedicatedSystemMemoryBytes() + GetDedicatedVideoMemoryBytes() + GetUnknownMemoryBytes();
}

FResourceSizeEx& FResourceSizeEx::operator+=(const FResourceSizeEx& InRHS)
{
	ensureAlwaysMsgf(ResourceSizeMode == InRHS.ResourceSizeMode, TEXT("The two resource sizes use different counting modes. The result of adding them together may be incorrect."));

	for (const TPair<FName, SIZE_T>& Pair : InRHS.DedicatedSystemMemoryBytesMap)
	{
		DedicatedSystemMemoryBytesMap.FindOrAdd(Pair.Key) += Pair.Value;
	}

	for (const TPair<FName, SIZE_T>& Pair : InRHS.DedicatedVideoMemoryBytesMap)
	{
		DedicatedVideoMemoryBytesMap.FindOrAdd(Pair.Key) += Pair.Value;
	}

	for (const TPair<FName, SIZE_T>& Pair : InRHS.UnknownMemoryBytesMap)
	{
		UnknownMemoryBytesMap.FindOrAdd(Pair.Key) += Pair.Value;
	}

	return *this;
}

FResourceSizeEx operator+(FResourceSizeEx InLHS, const FResourceSizeEx& InRHS)
{
	InLHS += InRHS;
	return InLHS;
}
