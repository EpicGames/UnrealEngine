// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ScopedDebugInfo.cpp: Scoped debug info implementation.
=============================================================================*/

#include "ProfilingDebugging/ScopedDebugInfo.h"

#include "HAL/PlatformTLS.h"
#include "Misc/AssertionMacros.h"

/** The TLS index for the debug info stack. */
static uint32 GThreadDebugInfoTLSID = FPlatformTLS::AllocTlsSlot();

FScopedDebugInfo::FScopedDebugInfo(int32 InNumReplacedOuterCalls):
	NumReplacedOuterCalls(InNumReplacedOuterCalls),
	NextOuterInfo((FScopedDebugInfo*)FPlatformTLS::GetTlsValue(GThreadDebugInfoTLSID))
{
	// Set the this debug info as the current innermost debug info.
	FPlatformTLS::SetTlsValue(GThreadDebugInfoTLSID,this);
}

FScopedDebugInfo::~FScopedDebugInfo()
{
	FScopedDebugInfo* CurrentInnermostDebugInfo = (FScopedDebugInfo*)FPlatformTLS::GetTlsValue(GThreadDebugInfoTLSID);
	check(CurrentInnermostDebugInfo == this);

	// Set the next outermost link to the current innermost debug info.
	FPlatformTLS::SetTlsValue(GThreadDebugInfoTLSID,NextOuterInfo);
}

FScopedDebugInfo* FScopedDebugInfo::GetDebugInfoStack()
{
	return (FScopedDebugInfo*)FPlatformTLS::GetTlsValue(GThreadDebugInfoTLSID);
}
