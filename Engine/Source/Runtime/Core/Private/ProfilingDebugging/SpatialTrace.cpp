// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProfilingDebugging/SpatialTrace.h"

#if UE_TRACE_SPATIAL_ENABLED
#include "HAL/PlatformTime.h"
#include "Trace/Trace.inl"

// Configuration for whether to trace only components which have changed.
#ifndef UE_TRACE_SPATIAL_COMPONENT_DELTAS
#define UE_TRACE_SPATIAL_COMPONENT_DELTAS 0
#endif // UE_TRACE_SPATIAL_COMPONENT_DELTAS

UE_TRACE_MINIMAL_CHANNEL_DEFINE(SpatialChannel, "Named spatial point data (position, velocity, direction) sampled over time. Allows tracking the spatial state of arbitrary points for debugging movement or physics.");

UE_TRACE_MINIMAL_EVENT_BEGIN(Spatial, PointSpec, NoSync | Important)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint32, Id)
	UE_TRACE_MINIMAL_EVENT_FIELD(UE::Trace::WideString, Name)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint8, Mask)
UE_TRACE_MINIMAL_EVENT_END()

// Helper macro containing component event boilerplate.
#define UE_TRACE_MINIMAL_SPATIAL_EVENT_BEGIN(LoggerName, EventName, ...) \
	UE_TRACE_MINIMAL_EVENT_BEGIN(Spatial, EventName, ##__VA_ARGS__) \
		UE_TRACE_MINIMAL_EVENT_FIELD(uint64, Cycle) \
		UE_TRACE_MINIMAL_EVENT_FIELD(uint32, SpecId)

// 2^N - 1 subsets (N = number of components, no empty subset).

// 1-component events.

UE_TRACE_MINIMAL_SPATIAL_EVENT_BEGIN(Spatial, SetPointPos)
	UE_TRACE_MINIMAL_EVENT_FIELD(double[], Pos)
UE_TRACE_MINIMAL_EVENT_END()

UE_TRACE_MINIMAL_SPATIAL_EVENT_BEGIN(Spatial, SetPointVel)
	UE_TRACE_MINIMAL_EVENT_FIELD(double[], Vel)
UE_TRACE_MINIMAL_EVENT_END()

UE_TRACE_MINIMAL_SPATIAL_EVENT_BEGIN(Spatial, SetPointDir)
	UE_TRACE_MINIMAL_EVENT_FIELD(double[], Dir)
UE_TRACE_MINIMAL_EVENT_END()

// 2-component events.

UE_TRACE_MINIMAL_SPATIAL_EVENT_BEGIN(Spatial, SetPointPosVel)
	UE_TRACE_MINIMAL_EVENT_FIELD(double[], Pos)
	UE_TRACE_MINIMAL_EVENT_FIELD(double[], Vel)
UE_TRACE_MINIMAL_EVENT_END()

UE_TRACE_MINIMAL_SPATIAL_EVENT_BEGIN(Spatial, SetPointPosDir)
	UE_TRACE_MINIMAL_EVENT_FIELD(double[], Pos)
	UE_TRACE_MINIMAL_EVENT_FIELD(double[], Dir)
UE_TRACE_MINIMAL_EVENT_END()

UE_TRACE_MINIMAL_SPATIAL_EVENT_BEGIN(Spatial, SetPointVelDir)
	UE_TRACE_MINIMAL_EVENT_FIELD(double[], Vel)
	UE_TRACE_MINIMAL_EVENT_FIELD(double[], Dir)
UE_TRACE_MINIMAL_EVENT_END()

// 3-component events.

UE_TRACE_MINIMAL_SPATIAL_EVENT_BEGIN(Spatial, SetPointPosVelDir)
	UE_TRACE_MINIMAL_EVENT_FIELD(double[], Pos)
	UE_TRACE_MINIMAL_EVENT_FIELD(double[], Vel)
	UE_TRACE_MINIMAL_EVENT_FIELD(double[], Dir)
UE_TRACE_MINIMAL_EVENT_END()

#undef UE_TRACE_MINIMAL_SPATIAL_EVENT_BEGIN

namespace UE::SpatialTrace::Private
{

	uint32 GetNextPointSpecId()
	{
		static std::atomic<uint32> NextPointSpecId = 1;
		return NextPointSpecId.fetch_add(1);
	}

	uint32 TracePointSpec(FStringView Name, EPointComponentFlags ComponentMask)
	{
		if (!UE_TRACE_CHANNELEXPR_IS_ENABLED(SpatialChannel))
		{
			return InvalidPointSpecId;
		}

		const uint32 SpecId = GetNextPointSpecId();

		UE_TRACE_MINIMAL_LOG(Spatial, PointSpec, SpatialChannel, static_cast<uint32>(Name.NumBytes()))
			<< PointSpec.Id(SpecId)
			<< PointSpec.Name(GetData(Name), GetNum(Name))
			<< PointSpec.Mask((uint8)ComponentMask);

		return SpecId;
	}

	void TracePointSet(uint32 SpecId, EPointComponentFlags SpecComponentMask, EPointComponentFlags ChangedComponentMask, const FPointDataView& ComponentData)
	{
		if (!UE_TRACE_CHANNELEXPR_IS_ENABLED(SpatialChannel))
		{
			return;
		}

#define UE_TRACE_MINIMAL_SPATIAL_LOG(LoggerName, EventName, ChannelsExpr, ...) \
			UE_TRACE_MINIMAL_LOG(LoggerName, EventName, ChannelsExpr, ##__VA_ARGS__) \
				<< EventName.Cycle(FPlatformTime::Cycles64()) \
				<< EventName.SpecId(SpecId)

		const EPointComponentFlags EventComponentMask = UE_TRACE_SPATIAL_COMPONENT_DELTAS ? ChangedComponentMask : SpecComponentMask;
		switch (EventComponentMask)
		{
			case EPointComponentFlags::Position:
				UE_TRACE_MINIMAL_SPATIAL_LOG(Spatial, SetPointPos, SpatialChannel)
					<< SetPointPos.Pos(&ComponentData.Position->X, 3);
				break;

			case EPointComponentFlags::Velocity:
				UE_TRACE_MINIMAL_SPATIAL_LOG(Spatial, SetPointVel, SpatialChannel)
					<< SetPointVel.Vel(&ComponentData.Velocity->X, 3);
				break;

			case EPointComponentFlags::Direction:
				UE_TRACE_MINIMAL_SPATIAL_LOG(Spatial, SetPointDir, SpatialChannel)
					<< SetPointDir.Dir(&ComponentData.Direction->X, 3);
				break;

			case EPointComponentFlags::Position | EPointComponentFlags::Velocity:
				UE_TRACE_MINIMAL_SPATIAL_LOG(Spatial, SetPointPosVel, SpatialChannel)
					<< SetPointPosVel.Pos(&ComponentData.Position->X, 3)
					<< SetPointPosVel.Vel(&ComponentData.Velocity->X, 3);
				break;

			case EPointComponentFlags::Position | EPointComponentFlags::Direction:
				UE_TRACE_MINIMAL_SPATIAL_LOG(Spatial, SetPointPosDir, SpatialChannel)
					<< SetPointPosDir.Pos(&ComponentData.Position->X, 3)
					<< SetPointPosDir.Dir(&ComponentData.Direction->X, 3);
				break;

			case EPointComponentFlags::Velocity | EPointComponentFlags::Direction:
				UE_TRACE_MINIMAL_SPATIAL_LOG(Spatial, SetPointVelDir, SpatialChannel)
					<< SetPointVelDir.Vel(&ComponentData.Velocity->X, 3)
					<< SetPointVelDir.Dir(&ComponentData.Direction->X, 3);
				break;

			case EPointComponentFlags::Position | EPointComponentFlags::Velocity | EPointComponentFlags::Direction:
				UE_TRACE_MINIMAL_SPATIAL_LOG(Spatial, SetPointPosVelDir, SpatialChannel)
					<< SetPointPosVelDir.Pos(&ComponentData.Position->X, 3)
					<< SetPointPosVelDir.Vel(&ComponentData.Velocity->X, 3)
					<< SetPointPosVelDir.Dir(&ComponentData.Direction->X, 3);
				break;

			default:
				checkNoEntry();
				break;
		}

#undef UE_TRACE_MINIMAL_SPATIAL_LOG
	}

	TTuple<uint32, bool> GetOrCreatePointSpecId(std::atomic<uint32>& InOutSpecId, FStringView Name, EPointComponentFlags ComponentMask)
	{
		const uint32 CurrentId = InOutSpecId.load(std::memory_order_relaxed);
		if (CurrentId != InvalidPointSpecId)
		{
			return MakeTuple(CurrentId, false);
		}

		const uint32 NewSpecId = TracePointSpec(Name, ComponentMask);
		uint32 Expected = InvalidPointSpecId;
		if (InOutSpecId.compare_exchange_strong(Expected, NewSpecId, std::memory_order_relaxed))
		{
			return MakeTuple(NewSpecId, true);
		}
		else
		{
			return MakeTuple(Expected, false);
		}
	}

} // namespace UE::SpatialTrace::Private

#endif // UE_TRACE_SPATIAL_ENABLED