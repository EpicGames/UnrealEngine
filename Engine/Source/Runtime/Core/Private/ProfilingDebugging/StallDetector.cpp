// Copyright Epic Games, Inc. All Rights Reserved.
#include "ProfilingDebugging/StallDetector.h"

#if STALL_DETECTOR

#include "HAL/ExceptionHandling.h"
#include "HAL/PlatformProcess.h"
#include "HAL/PlatformStackWalk.h"
#include "HAL/PlatformTLS.h"
#include "HAL/PlatformTime.h"
#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "Misc/FeedbackContext.h"
#include "Tasks/Task.h"
#include "Templates/Greater.h"

// counters for sending information into trace system
TRACE_DECLARE_INT_COUNTER(StallCount, TEXT("StallDetector/Count"));
TRACE_DECLARE_FLOAT_COUNTER(StallTimeSeconds, TEXT("StallDetector/TimeSeconds"));

// force normal behavior in the face of debug configuration and debugger attached
#define STALL_DETECTOR_DEBUG 0

// use the heart beat clock to account for process suspend
#define STALL_DETECTOR_HEART_BEAT_CLOCK 1

#if STALL_DETECTOR_HEART_BEAT_CLOCK
 #include "HAL/ThreadHeartBeat.h"
#endif // STALL_DETECTOR_HEART_BEAT_CLOCK

DEFINE_LOG_CATEGORY(LogStall);

/**
* Globals
**/

// The reference count for the resources for this API
static int32 InitCount = 0;

// Sentinel value for invalid timestamp
static const double InvalidSeconds = -1.0;

UE::FOnStallDetected UE::FStallDetector::StallDetected;
UE::FOnStallCompleted UE::FStallDetector::StallCompleted;

// Scopes currently being tracked across all threads
static FCriticalSection StallScopesSection;
static TSet<UE::FStallDetector*> StallScopes;

static bool ForceStallDetectedLog = false;
static FAutoConsoleVariableRef CVarForceStallDetectedLog(
	TEXT("StallDetector.ForceLogOnStall"),
	ForceStallDetectedLog,
	TEXT("Forces StallDetector to make a log to LogStall at verbosity=Log even if reporting mode is disabled"),
	ECVF_SetByConsole);

/**
* Stall Detector Thread
**/

namespace UE
{
	struct FDetectedStall
	{
		EStallDetectorReportingMode ReportingMode;
		TArray<uint64> Backtrace;
		uint32 ThreadId;
		const TCHAR* Name;
		double BudgetSeconds;
		double OverageSeconds;
		uint64 UniqueId;
		bool bReported;
	};

	class FStallDetectorRunnable : public FRunnable
	{
	public:
		FStallDetectorRunnable();

		// FRunnable implementation
		virtual uint32 Run() override;
		virtual void Stop() override
		{
			StopThread = true;
		}
		virtual void Exit() override
		{
			Stop();
		}

		bool GetStartedThread() const
		{
			return StartedThread.load(std::memory_order_relaxed);
		}
		
#if STALL_DETECTOR_HEART_BEAT_CLOCK
		FThreadHeartBeatClock& GetClock()
		{
			return Clock;
		}
#endif

	private:
		std::atomic<bool> StartedThread;
		bool StopThread;

#if STALL_DETECTOR_HEART_BEAT_CLOCK
		FThreadHeartBeatClock Clock;
#endif
	};

	static FStallDetectorRunnable* StallDetectorRunnable = nullptr;
	static FRunnableThread* StallDetectorThread = nullptr;
}

UE::FStallDetectorRunnable::FStallDetectorRunnable()
	: StartedThread(false)
	, StopThread(false)
#if STALL_DETECTOR_HEART_BEAT_CLOCK
	, Clock(50.0/1000.0) // the clamped time interval that each tick of the clock can possibly advance
#endif
{
}

uint32 UE::FStallDetectorRunnable::Run()
{
	while (!StopThread)
	{
#if STALL_DETECTOR_HEART_BEAT_CLOCK
		Clock.Tick();
#endif

		// Clock has been ticked
		StartedThread = true;

		// Use this timestamp to try to avoid marginal triggering
		double Seconds = FStallDetector::Seconds();

		// Check the detectors
		if (Seconds != InvalidSeconds)
		{
			bool bDisableReporting = UE_BUILD_DEBUG;
#if !STALL_DETECTOR_DEBUG
			bDisableReporting = bDisableReporting || FPlatformMisc::IsDebuggerPresent(); // Do not generate a report if we detect the debugger mucking with things
#endif

			TArray<FDetectedStall> DetectedStalls;
			FScopeLock ScopeLock(&StallScopesSection);

			// Stop if there has been a crash, do not report crash as stall, Detector->Stats may also be deleted memory
			if (GIsCriticalError)
			{
				break;
			}

			for (FStallDetector* Detector : StallScopes)
			{
				if (Detector->bTriggered)
				{
					continue;
				}

				double OverageSeconds = Detector->Check(false, Seconds);
				if (OverageSeconds > 0.0 && !FPlatformMisc::IsDebuggerPresent())
				{
					TArray<uint64> Backtrace;
					Backtrace.AddUninitialized(StallDetectorMaxBacktraceEntries);
					int32 NumStackEntries = FPlatformStackWalk::CaptureThreadStackBackTrace(Detector->ThreadId, Backtrace.GetData(), Backtrace.Num());
					Backtrace.SetNum(NumStackEntries);

					// Store enough info to fire the event when we are no longer holding the lock
					DetectedStalls.Emplace(FDetectedStall{
						bDisableReporting ? EStallDetectorReportingMode::Disabled : Detector->Stats.ReportingMode,
						MoveTemp(Backtrace),
						Detector->ThreadId,
						Detector->Stats.Name,
						Detector->Stats.BudgetSeconds,
						OverageSeconds,
						Detector->UniqueID,
						Detector->Stats.bReported });

					// Triggered can be reset, reported cannot
					Detector->bTriggered = true;
					Detector->Stats.bReported = true;
				}
			}

			// Do stall reporting in a task to keep heartbeat ticking for FStallDetector::Seconds resolution
			UE::Tasks::Launch(UE_SOURCE_LOCATION, [DetectedStalls = MoveTemp(DetectedStalls)]() {
				// Determine reporting requirements for each detected stall
				for (const FDetectedStall& Stall : DetectedStalls)
				{
					bool bSendReport = false;
					switch (Stall.ReportingMode)
					{
						case EStallDetectorReportingMode::First:
							bSendReport = !Stall.bReported;
							break;

						case EStallDetectorReportingMode::Always:
							bSendReport = true;
							break;

						default:
							break;
					}

					if (bSendReport)
					{
						FStallDetectorStats::TotalReportedCount.Increment();
						const int NumStackFramesToIgnore = 0;
						UE_LOGF(LogStall, Log, "Stall detector '%ls' exceeded budget of %fs, reporting...", Stall.Name, Stall.BudgetSeconds);
						double ReportSeconds = FPlatformTime::Seconds();
						ReportStall(Stall.Name, Stall.ThreadId);
						ReportSeconds = FPlatformTime::Seconds() - ReportSeconds;
						UE_LOGF(LogStall, Log, "Stall detector '%ls' report submitted, and took %fs", Stall.Name, ReportSeconds);
					}
					else if (ForceStallDetectedLog || Stall.ReportingMode != EStallDetectorReportingMode::Disabled)
					{
						UE_LOGF(LogStall, Log, "Stall detector '%ls' exceeded budget of %fs", Stall.Name, Stall.BudgetSeconds);
					}
				}

				// Fire external notifications
				//
				for (const FDetectedStall& Stall : DetectedStalls)
				{
					FStallDetector::StallDetected.Broadcast(FStallDetectedParams{ Stall.Name, Stall.ThreadId, Stall.UniqueId, Stall.Backtrace });
				}
			});
		}

		// Sleep an interval, the resolution at which we want to detect an overage
		FPlatformProcess::SleepNoStats(0.005f);
	}

	return 0;
}

/**
* Stall Detector Stats
**/

static FCriticalSection StallDetectorStatsCritical;
static TSet<UE::FStallDetectorStats*> StallDetectorStats;
FCountersTrace::FCounterAtomicInt UE::FStallDetectorStats::TotalTriggeredCount (TEXT("StallDetector/TotalTriggeredCount"), TraceCounterDisplayHint_None);
FCountersTrace::FCounterAtomicInt UE::FStallDetectorStats::TotalReportedCount (TEXT("StallDetector/TotalReportedCount"), TraceCounterDisplayHint_None);

UE::FStallDetectorStats::FStallDetectorStats(const TCHAR* InName, const double InBudgetSeconds, const EStallDetectorReportingMode InReportingMode)
	: Name(InName)
	, BudgetSeconds(InBudgetSeconds)
	, ReportingMode(InReportingMode)
	, bReported(false)
	, TriggerCount(
		TraceCounterNameType_Static,
		(
			FCString::Strcat(TriggerCountCounterName, TEXT("StallDetector/")),
			FCString::Strcat(TriggerCountCounterName, InName),
			FCString::Strcat(TriggerCountCounterName, TEXT(" TriggerCount"))
		), TraceCounterDisplayHint_None)
	, OverageSeconds(
		TraceCounterNameType_Static,
		(
			FCString::Strcat(OverageSecondsCounterName, TEXT("StallDetector/")),
			FCString::Strcat(OverageSecondsCounterName, InName),
			FCString::Strcat(OverageSecondsCounterName, TEXT(" OverageSeconds"))
		), TraceCounterDisplayHint_None)
{
	// Add at the end of construction
	FScopeLock ScopeLock(&StallDetectorStatsCritical);
	StallDetectorStats.Add(this);
}

UE::FStallDetectorStats::~FStallDetectorStats()
{
	// Remove at the beginning of destruction
	FScopeLock ScopeLock(&StallDetectorStatsCritical);
	StallDetectorStats.Remove(this);
}

void UE::FStallDetectorStats::OnStallCompleted(double InOverageSeconds)
{
	// we sync access around these for coherency reasons, can be polled from another thread (tabulation)
	FScopeLock StatsLock(&StatsSection);
	TriggerCount.Increment();
	OverageSeconds.Add(InOverageSeconds);
}

void UE::FStallDetectorStats::TabulateStats(TArray<TabulatedResult>& TabulatedResults)
{
	TabulatedResults.Empty();

	TArray<TabulatedResult> StatsArray;
	{
		FScopeLock ScopeLock(&StallDetectorStatsCritical);
		for (const UE::FStallDetectorStats* StallStats : StallDetectorStats)
		{
			if (StallStats->TriggerCount.Get() && StallStats->ReportingMode != UE::EStallDetectorReportingMode::Disabled)
			{
				// we sync access around these for coherency reasons, can be polled from another thread (detector or scope)
				FScopeLock Lock(&StallStats->StatsSection);
				StatsArray.Emplace(TabulatedResult{ StallStats->Name, StallStats->BudgetSeconds, StallStats->TriggerCount.Get(), StallStats->OverageSeconds.Get(), 0.0 });
				StatsArray.Last().OverageRatio = (StallStats->OverageSeconds.Get() / (double)StallStats->TriggerCount.Get()) / StallStats->BudgetSeconds;
			}
		}
	}

	if (!StatsArray.IsEmpty())
	{
		Algo::SortBy(
			StatsArray,
			[](const TabulatedResult& Result) { return Result.OverageRatio; },
			TGreater<double>{});
		TabulatedResults = MoveTemp(StatsArray);
	}
}


/**
* Stall Timer
**/

UE::FStallTimer::FStallTimer()
: PauseCount(0)
, LastCheckSeconds(InvalidSeconds)
, RemainingSeconds(InvalidSeconds)
{
}

void UE::FStallTimer::Reset(const double InSeconds, const double InRemainingSeconds)
{
	FScopeLock ScopeLock(&Section);

	LastCheckSeconds = InSeconds;
	RemainingSeconds = InRemainingSeconds;
}

void UE::FStallTimer::Check(const double InSeconds, double& OutDeltaSeconds, double& OutOverageSeconds)
{
	OutDeltaSeconds = 0.0;
	OutOverageSeconds = 0.0;

	FScopeLock ScopeLock(&Section);

	// Check to see if we are paused, no need to check in this case
	if (LastCheckSeconds == InvalidSeconds)
	{
		return;
	}

	// The amount of time that has transpired since the last check
	OutDeltaSeconds = InSeconds - LastCheckSeconds;

	// Update the last check time
	LastCheckSeconds = InSeconds;

	// Deduct our delta from the remaining time
	RemainingSeconds -= OutDeltaSeconds;

	// The following calls use overage vs. negative remaining time
	OutOverageSeconds = -RemainingSeconds;
}

void UE::FStallTimer::Pause(const double InSeconds)
{
	FScopeLock ScopeLock(&Section);

	if (++PauseCount == 1 && ensure(LastCheckSeconds != InvalidSeconds))
	{
		// Keep remaining time up to date with now
		RemainingSeconds -= InSeconds - LastCheckSeconds;

		// Forget when we start this interval, we don't need synchronization here as if Check() updates LastCheckSeconds between the above and here, it doesn't matter as RemainingSeconds will still be correct
		LastCheckSeconds = InvalidSeconds;
	}
}

void UE::FStallTimer::Resume(const double InSeconds)
{
	FScopeLock ScopeLock(&Section);

	if (--PauseCount == 0)
	{
		// We should be paused!
		ensure(LastCheckSeconds == InvalidSeconds);

		// Resume checking from this time, we don't need atomics here as this thread is the only one that should issue this write (we were paused which means the StallDetectorThread won't Check())
		LastCheckSeconds = InSeconds;
	}
}


/**
* Stall Detector
**/

// globals
class ThreadLocalSlotAcquire
{
public:
	ThreadLocalSlotAcquire()
	{
		Slot = FPlatformTLS::AllocTlsSlot();
		FPlatformTLS::SetTlsValue(Slot, nullptr);
	}

	~ThreadLocalSlotAcquire()
	{
		FPlatformTLS::FreeTlsSlot(Slot);
		Slot = FPlatformTLS::InvalidTlsSlot;
	}

	uint32 GetSlot()
	{
		return Slot;
	}

private:
	uint32 Slot = FPlatformTLS::InvalidTlsSlot;
};
static ThreadLocalSlotAcquire ThreadLocalSlot;

TAtomic<uint64> UE::FStallDetector::UIDGenerator = 0;

/**
 * Pause on slow task
 */
namespace UE::StallDetector::Private
{
	FDelegateHandle GSlowTaskStartDelegate;
	FDelegateHandle GSlowTaskFinalizedDelegate;

	void RegisterSlowTaskHandlers();
	void UnregisterSlowTaskHandlers();
}

UE::FStallDetector::FStallDetector(FStallDetectorStats& InStats)
	: Stats(InStats)
	, Parent(nullptr)
	, ThreadId(0)
	, bStarted(false)
	, bPersistent(false)
	, bTriggered(false)
	, UniqueID(UIDGenerator.IncrementExchange())

{
	// Capture this thread's current stall detector to our parent member
	Parent = static_cast<FStallDetector*>(FPlatformTLS::GetTlsValue(ThreadLocalSlot.GetSlot()));

	// Set the current thread's stall detector to this object
	FPlatformTLS::SetTlsValue(ThreadLocalSlot.GetSlot(), this);

	// This will be invalid if the stall detector thread isn't running
	double Seconds = FStallDetector::Seconds();
	if (Seconds != InvalidSeconds)
	{
		// Capture the calling thread's id, can't do it later because we could need it from the stall detector thread
		ThreadId = FPlatformTLS::GetCurrentThreadId();

		// Setting this will cause our dtor to do a check (unless we are put into persistent mode by CheckAndReset())
		Timer.Reset(Seconds, InStats.BudgetSeconds);

		// Flag this detector to be checked at destruction time
		bStarted = true;
	}

	// Add at the end of construction
	FScopeLock ScopeLock(&StallScopesSection);
	StallScopes.Add(this);
}

UE::FStallDetector::~FStallDetector()
{
	// Remove at the beginning of destruction
	{
		FScopeLock ScopeLock(&StallScopesSection);
		StallScopes.Remove(this);
	}

	// This will be invalid if the stall detector thread isn't running
	double Seconds = FStallDetector::Seconds();

	// If we have a timestamp, and captured a timestamp at contruction time, and we are not a persistent detector, do the final check
	if (Seconds != InvalidSeconds && bStarted && !bPersistent)
	{
		double OverageSeconds = Check(true, Seconds);

		if (OverageSeconds > 0.0)
		{
#if STALL_DETECTOR_DEBUG
			FString OverageString = FString::Printf(TEXT("[FStallDetector] [%s] Overage of %f\n"), Stats.Name, OverageSeconds);
			FPlatformMisc::LocalPrint(OverageString.GetCharArray().GetData());
#endif
			if (Stats.ReportingMode != EStallDetectorReportingMode::Disabled)
			{
				UE_LOGF(LogStall, Log, "Stall detector '%ls' complete in %fs (%fs overbudget)", Stats.Name, Stats.BudgetSeconds + OverageSeconds, OverageSeconds);
			}
			Stats.OnStallCompleted(OverageSeconds);
			StallCompleted.Broadcast(FStallCompletedParams{ ThreadId, Stats.Name, Stats.BudgetSeconds, OverageSeconds, UniqueID, bTriggered });
		}
	}

	// We are destructed, so set current thread's detector to our parent
	FPlatformTLS::SetTlsValue(ThreadLocalSlot.GetSlot(), Parent);
}

double UE::FStallDetector::Check(bool bFinalCheck, double InCheckSeconds)
{
	// all callers need to sort out checking the clock
	check(InCheckSeconds != InvalidSeconds);

	// Both the potentially stalling thread and the stall detector thread will call into here.
	// FStallTimer will take a section and calculate the time remaining and update the last checked time stamp.
	// We don't hammer this too too badly (5ms at the time of this writing), so the lock shouldn't be too costly.
	// There shouldn't be enough contention to warrant trying to implement these semantics via atomics.
	// The critical section fastpath should be in effect almost all the time.
	double DeltaSeconds;
	double OverageSeconds;
	Timer.Check(InCheckSeconds, DeltaSeconds, OverageSeconds);
	if (OverageSeconds > 0.0)
	{
		Stats.TotalTriggeredCount.Increment();
	}
	return OverageSeconds;
}

void UE::FStallDetector::CheckAndReset()
{
	double CheckSeconds = FStallDetector::Seconds();
	if (CheckSeconds == InvalidSeconds)
	{
		return;
	}

	// Lock to prevent races over bTriggered flag
	FScopeLock ScopeLock(&StallScopesSection);

	bool bExceeded = false;
	// If this is the first call to CheckAndReset, flag that we are now in persistent mode (vs. scope mode)
	if (!bPersistent)
	{
		bPersistent = true;
	}
	else
	{
		// only perform the check on the second call
		double OverageSeconds = Check(true, CheckSeconds);
		if (OverageSeconds > 0.0)
		{
#if STALL_DETECTOR_DEBUG
			FString OverageString = FString::Printf(TEXT("[FStallDetector] [%s] Persistent stall detector overage of %f\n"), Stats.Name, OverageSeconds);
			FPlatformMisc::LocalPrint(OverageString.GetCharArray().GetData());
#endif
			if (Stats.ReportingMode != EStallDetectorReportingMode::Disabled)
			{
				UE_LOGF(LogStall, Log, "Stall detector '%ls' complete in %fs (%fs overbudget)", Stats.Name, Stats.BudgetSeconds + OverageSeconds, OverageSeconds);
			}
			Stats.OnStallCompleted(OverageSeconds);
			StallCompleted.Broadcast(FStallCompletedParams{ ThreadId, Stats.Name, Stats.BudgetSeconds, OverageSeconds, UniqueID, bTriggered });
		}
	}

	Timer.Reset(CheckSeconds, Stats.BudgetSeconds);
	bTriggered = false;
	UniqueID = UIDGenerator.IncrementExchange();
}

void UE::FStallDetector::Pause(const double InSeconds)
{
	// All callers need to sort out checking the clock
	check(InSeconds != InvalidSeconds);

	Timer.Pause(InSeconds);
}

void UE::FStallDetector::Resume(const double InSeconds)
{
	// All callers need to sort out checking the clock
	check(InSeconds != InvalidSeconds);

	Timer.Resume(InSeconds);
}

double UE::FStallDetector::Seconds()
{
	double Result = InvalidSeconds;

	if (FStallDetector::IsRunning())
	{
#if STALL_DETECTOR_HEART_BEAT_CLOCK
		Result = StallDetectorRunnable->GetClock().Seconds();
#else
		Result = FPlatformTime::Seconds();
#endif

#if STALL_DETECTOR_DEBUG
		static double ClockStartSeconds = Result;
		static double PlatformStartSeconds = FPlatformTime::Seconds();
		double ClockDelta = Result - ClockStartSeconds;
		double PlatformDelta = FPlatformTime::Seconds() - PlatformStartSeconds;
		double Drift = PlatformDelta - ClockDelta;
		static double LastDrift = Drift;
		double DriftDelta = Drift - LastDrift;
		if (DriftDelta > 0.001)
		{
			FString ResultString = FString::Printf(TEXT("[FStallDetector] Thread %5d / Platform: %f / Clock: %f / Drift: %f (%f)\n"), FPlatformTLS::GetCurrentThreadId(), PlatformDelta, ClockDelta, Drift, DriftDelta);
			FPlatformMisc::LocalPrint(ResultString.GetCharArray().GetData());
			LastDrift = Drift;
		}
#endif
	}

	return Result;
}

void UE::FStallDetector::Startup()
{
	check(InitCount >= 0);
	if (++InitCount == 1)
	{
		UE_LOGF(LogStall, Log, "Startup...");

		check(FPlatformTime::GetSecondsPerCycle() > 0.0);

		// Cannot be a global due to clock member
		StallDetectorRunnable = new FStallDetectorRunnable();

		if (FPlatformProcess::SupportsMultithreading())
		{
			if (StallDetectorThread == nullptr)
			{
				StallDetectorThread = FRunnableThread::Create(StallDetectorRunnable, TEXT("StallDetectorThread"));
				check(StallDetectorThread);

				// Poll until we have ticked the clock
				while (!StallDetectorRunnable->GetStartedThread())
				{
					FPlatformProcess::YieldThread();
				}

				StallDetector::Private::RegisterSlowTaskHandlers();
			}
		}

		UE_LOGF(LogStall, Log, "Startup complete.");
	}
}

void UE::FStallDetector::Shutdown()
{
	if (--InitCount == 0)
	{
		UE_LOGF(LogStall, Log, "Shutdown...");

		StallDetector::Private::UnregisterSlowTaskHandlers();

		delete StallDetectorThread;
		StallDetectorThread = nullptr;

		delete StallDetectorRunnable;
		StallDetectorRunnable = nullptr;

		UE_LOGF(LogStall, Log, "Shutdown complete.");
	}
	check(InitCount >= 0);
}

bool UE::FStallDetector::IsRunning()
{
	return InitCount > 0;
}

UE::FStallDetectorPause::FStallDetectorPause()
	: bPaused(false)
{
	// this will be invalid if the thread isn't running
	double Seconds = FStallDetector::Seconds();
	if (Seconds == InvalidSeconds)
	{
		return;
	}

	bPaused = true;

	for (FStallDetector* Current = static_cast<FStallDetector*>(FPlatformTLS::GetTlsValue(ThreadLocalSlot.GetSlot())); Current; Current = Current->GetParent())
	{
		Current->Pause(Seconds);
	}
}

UE::FStallDetectorPause::~FStallDetectorPause()
{
	// this will be invalid if the thread isn't running
	double Seconds = FStallDetector::Seconds();
	if (Seconds == InvalidSeconds)
	{
		return;
	}

	if (bPaused)
	{
		for (FStallDetector* Current = static_cast<FStallDetector*>(FPlatformTLS::GetTlsValue(ThreadLocalSlot.GetSlot())); Current; Current = Current->GetParent())
		{
			Current->Resume(Seconds);
		}
	}
}

bool UE::FStallDetectorPause::IsPaused() const
{
	return bPaused;
}


/**
 * Pause on slow task
 */
namespace UE::StallDetector::Private
{
	static ThreadLocalSlotAcquire SlowTaskPauseSlot;

	/**
	 * A StallDetector pause context which handles cases where SlowTask is invoked
	 * Each pause context forms a node in a linked list - the tail is held in the SlowTaskPauseSlot TLS slot
	 */
	class FSlowTaskStallDetectorPause : public FStallDetectorPause
	{
	public:
		explicit FSlowTaskStallDetectorPause(FSlowTaskStallDetectorPause* InParent);
		FSlowTaskStallDetectorPause* GetParent() const;
	private:
		FSlowTaskStallDetectorPause* Parent;
	};
	
	FSlowTaskStallDetectorPause::FSlowTaskStallDetectorPause(FSlowTaskStallDetectorPause* InParent)
		: FStallDetectorPause()
		, Parent(InParent)
	{
	}

	FSlowTaskStallDetectorPause* FSlowTaskStallDetectorPause::GetParent() const
	{
		return Parent;
	}

	static void HandleSlowTaskStart(const FText& TaskName)
	{
		FSlowTaskStallDetectorPause* Parent = static_cast<FSlowTaskStallDetectorPause*>(FPlatformTLS::GetTlsValue(SlowTaskPauseSlot.GetSlot()));
		FSlowTaskStallDetectorPause* PauseState = new FSlowTaskStallDetectorPause(Parent);
		FPlatformTLS::SetTlsValue(SlowTaskPauseSlot.GetSlot(), PauseState);
	}

	static void HandleSlowTaskFinalize(const FText& TaskName, double DurationInSeconds)
	{
		FSlowTaskStallDetectorPause* PauseState = static_cast<FSlowTaskStallDetectorPause*>(FPlatformTLS::GetTlsValue(SlowTaskPauseSlot.GetSlot()));
		FSlowTaskStallDetectorPause* Parent = ensure(PauseState) ? PauseState->GetParent() : nullptr;
		FPlatformTLS::SetTlsValue(SlowTaskPauseSlot.GetSlot(), Parent);
		delete PauseState;
	}
	
	void RegisterSlowTaskHandlers()
	{
		GSlowTaskStartDelegate = GWarn->OnStartSlowTask().AddStatic(&HandleSlowTaskStart);
		GSlowTaskFinalizedDelegate = GWarn->OnFinalizeSlowTask().AddStatic(&HandleSlowTaskFinalize);
	}
	
	void UnregisterSlowTaskHandlers()
	{
		GWarn->OnFinalizeSlowTask().Remove(GSlowTaskFinalizedDelegate);
		GWarn->OnStartSlowTask().Remove(GSlowTaskStartDelegate);
	}
}

#endif // STALL_DETECTOR