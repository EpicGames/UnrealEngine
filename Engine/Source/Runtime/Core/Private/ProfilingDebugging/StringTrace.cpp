// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProfilingDebugging/StringsTrace.h"

#include "Containers/Set.h"
#include "HAL/CriticalSection.h"
#include "HAL/PlatformString.h"
#include "Misc/ScopeLock.h"
#include "ProfilingDebugging/StringsTrace.h"
#include "Trace/Trace.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

//////////////////////////////////////////////////////////////////////////////////////////
UE_TRACE_EVENT_DEFINE(Strings, StaticString);
UE_TRACE_EVENT_DEFINE(Strings, FName);

#if UE_TRACE_ENABLED

//////////////////////////////////////////////////////////////////////////////////////////
static struct FStaticStringTracer
{
	UE::Trace::FEventRef64 TraceString(const ANSICHAR* InString)
	{
		FScopeLock _(&LookupLock);
		uint64 Id = uint64(InString);
		bool bAlreadyInSet = false;
		TracedAnsiStrings.FindOrAdd(Id, &bAlreadyInSet);
		if (!bAlreadyInSet)
		{
			UE_TRACE_LOG_DEFINITION(Strings, StaticString, Id, true)
				 << StaticString.DisplayAnsi(InString, FCStringAnsi::Strlen(InString));
		}
		return UE::Trace::FEventRef64(Id, UE_TRACE_GET_DEFINITION_TYPE_ID(Strings, StaticString));
	}

	UE::Trace::FEventRef64 TraceString(const TCHAR* InString)
	{
		FScopeLock _(&LookupLock);
		uint64 Id = uint64(InString);
		bool bAlreadyInSet = false;
		TracedWideStrings.FindOrAdd(Id, &bAlreadyInSet);
		if (!bAlreadyInSet)
		{
			UE_TRACE_LOG_DEFINITION(Strings, StaticString, Id, true)
				 << StaticString.DisplayWide(InString, FCString::Strlen(InString));
		}
		return UE::Trace::FEventRef64(Id, UE_TRACE_GET_DEFINITION_TYPE_ID(Strings, StaticString));
	}
	
	void TraceStringsOnConnection()
	{
		FScopeLock _(&LookupLock);
		// Retrace all strings that has been referenced previously. Note that we create the reference id
		// manually so that we get the correct type id for sync version.
		for(const uint64 Ptr : TracedWideStrings)
		{
			const TCHAR* String = (const TCHAR*) Ptr;
			UE_TRACE_LOG_DEFINITION(Strings, StaticString, Ptr, true)
				<< StaticString.DisplayWide(String, FCString::Strlen(String));
		}
		for(const uint64 Ptr : TracedAnsiStrings)
		{
			const ANSICHAR* String = (const ANSICHAR*) Ptr;
			UE_TRACE_LOG_DEFINITION(Strings, StaticString, Ptr, true)
				<< StaticString.DisplayAnsi(String, FCStringAnsi::Strlen(String));
		}
	}

	FCriticalSection LookupLock;
	TSet<uint64> TracedWideStrings;
	TSet<uint64> TracedAnsiStrings;
} GStaticStringTracer;

//////////////////////////////////////////////////////////////////////////////////////////
UE::Trace::FEventRef32 FStringTrace::GetNameRef(const FName& Name)
{
	return FName::TraceName(Name);
}

//////////////////////////////////////////////////////////////////////////////////////////
UE::Trace::FEventRef64 FStringTrace::GetStaticStringRef(const TCHAR* String)
{
	return GStaticStringTracer.TraceString(String);
}

//////////////////////////////////////////////////////////////////////////////////////////
UE::Trace::FEventRef64 FStringTrace::GetStaticStringRef(const ANSICHAR* String)
{
	return GStaticStringTracer.TraceString(String);
}

//////////////////////////////////////////////////////////////////////////////////////////
void FStringTrace::OnConnection()
{
	GStaticStringTracer.TraceStringsOnConnection();
	FName::TraceNamesOnConnection();
}

#endif //UE_TRACE_ENABLED