// Copyright Epic Games, Inc. All Rights Reserved.
#include "ProfilingDebugging/TagTrace.h"

#include "Experimental/Containers/GrowOnlyLockFreeHash.h"
#include "Containers/StringView.h"
#include "CoreTypes.h"
#include "ProfilingDebugging/MemoryTrace.h"
#include "HAL/LLM/LLMPrivate.h"
#include "HAL/LowLevelMemTracker.h"
#include "HAL/PlatformString.h"
#include "Misc/Optional.h"
#include "Misc/StringBuilder.h"
#include "Trace/Trace.inl"
#include "UObject/NameTypes.h"

#if UE_MEMORY_TAGS_TRACE_ENABLED && UE_TRACE_ENABLED

////////////////////////////////////////////////////////////////////////////////

UE_TRACE_CHANNEL_EXTERN(MemAllocChannel);

UE_TRACE_EVENT_BEGIN(Memory, TagSpec, Important|NoSync)
	UE_TRACE_EVENT_FIELD(int32, Tag)
	UE_TRACE_EVENT_FIELD(int32, Parent)
	UE_TRACE_EVENT_FIELD(UE::Trace::AnsiString, Display)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Memory, MemoryScope, NoSync)
	UE_TRACE_EVENT_FIELD(int32, Tag)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Memory, MemoryScopePtr, NoSync)
	UE_TRACE_EVENT_FIELD(uint64, Ptr)
	UE_TRACE_EVENT_FIELD(uint8, RootHeap) // added in UE 5.6
UE_TRACE_EVENT_END()



////////////////////////////////////////////////////////////////////////////////
// Per thread active tag, i.e. the top level FMemScope
thread_local int32 GActiveTag;

////////////////////////////////////////////////////////////////////////////////
FMemScope::FMemScope()
{
}

FMemScope::FMemScope(int32 InTag, bool bShouldActivate /*= true*/)
{
	if (UE_TRACE_CHANNELEXPR_IS_ENABLED(MemAllocChannel) & bShouldActivate)
	{
		ActivateScope(InTag);
	}
}

////////////////////////////////////////////////////////////////////////////////
FMemScope::FMemScope(ELLMTag InTag, bool bShouldActivate /*= true*/)
{
	if (UE_TRACE_CHANNELEXPR_IS_ENABLED(MemAllocChannel) & bShouldActivate)
	{
		ActivateScope(static_cast<int32>(InTag));
	}
}

////////////////////////////////////////////////////////////////////////////////
FMemScope::FMemScope(const FName& InName, bool bShouldActivate /*= true*/)
{
	SetTagAndActivate(InName, bShouldActivate);
}

void FMemScope::SetTagAndActivate(const FName& InName, bool bShouldActivate /*= true*/)
{
	if (UE_TRACE_CHANNELEXPR_IS_ENABLED(MemAllocChannel) & bShouldActivate)
	{
		ActivateScope(MemoryTrace_AnnounceFNameTag(InName));
	}
}

////////////////////////////////////////////////////////////////////////////////
FMemScope::FMemScope(const UE::LLMPrivate::FTagData* TagData, bool bShouldActivate /*= true*/)
{
#if ENABLE_LOW_LEVEL_MEM_TRACKER
	if (UE_TRACE_CHANNELEXPR_IS_ENABLED(MemAllocChannel) & bShouldActivate && TagData)
	{
		ActivateScope(MemoryTrace_AnnounceFNameTag(TagData->GetName()));
	}
#endif // ENABLE_LOW_LEVEL_MEM_TRACKER
}

////////////////////////////////////////////////////////////////////////////////
void FMemScope::ActivateScope(int32 InTag)
{
	if (auto LogScope = FMemoryMemoryScopeFields::LogScopeType::ScopedEnter<FMemoryMemoryScopeFields>())
	{
		if (const auto& __restrict MemoryScope = *(FMemoryMemoryScopeFields*)(&LogScope))
		{
			Inner.SetActive();
			LogScope += LogScope
				<< MemoryScope.Tag(InTag);
			PrevTag = GActiveTag;
			GActiveTag = InTag;
		}
	}
}

////////////////////////////////////////////////////////////////////////////////
FMemScope::~FMemScope()
{
	if (Inner.bActive)
	{
		GActiveTag = PrevTag;
	}
}

////////////////////////////////////////////////////////////////////////////////
FMemScopePtr::FMemScopePtr(uint64 InPtr, HeapId InRootHeap)
{
	if (InPtr != 0 && TRACE_PRIVATE_CHANNELEXPR_IS_ENABLED(MemAllocChannel))
	{
		if (auto LogScope = FMemoryMemoryScopePtrFields::LogScopeType::ScopedEnter<FMemoryMemoryScopePtrFields>())
		{
			if (const auto& __restrict MemoryScope = *(FMemoryMemoryScopePtrFields*)(&LogScope))
			{
				Inner.SetActive();
				LogScope += LogScope
					<< MemoryScope.Ptr(InPtr)
					<< MemoryScope.RootHeap(uint8(InRootHeap));
			}
		}
	}
}

/////////////////////////////////////////////////////////////////////////////////
FMemScopePtr::~FMemScopePtr()
{
}

/////////////////////////////////////////////////////////////////////////////////

/**
 * Utility class that manages tracing the specification of unique LLM tags
 * and custom name based tags.
 */
class FTagTrace
{
public:
					FTagTrace(FMalloc* InMalloc);
	void			AnnounceGenericTags() const;
	void			AnnounceLLMExtendedTags();
	void			AnnounceSpecialTags() const;
#if ENABLE_LOW_LEVEL_MEM_TRACKER
	static void		OnLLMInitialised(UPTRINT pThis);
	static void		OnAnnounceLLMExtendedTag(const UE::LLMPrivate::FTagData* LLMTag, UPTRINT pThis);
	int32			AnnounceLLMExtendedTag(FLowLevelMemTracker& Tracker, const UE::LLMPrivate::FTagData* LLMTag);
#endif
	int32			AnnounceCustomTag(int32 Tag, int32 ParentTag, const ANSICHAR* Display) const;
	/**
	 * Send information about the given Tag to the trace if it has not already been sent.
	 *
	 * @param UniqueName The uniquename of the tag, guaranteed unique across all other memtrace tags.
	 * @param TagId The tag to use, or -1 to construct the Tag from UniqueName.
	 * @param DisplayName Displaystring of the tag when viewing the trace, or nullptr to use UniqueName.
	 * @param ParentTagId If set, the tag's parent tag, or -1 if the tag has no parent. If not set, ParentTag
	 *        is calculated by looking for markers in the UniqueName.
	 * @return The tag that ended up being used for the FName.
	 */
	int32 			AnnounceFNameTag(FName UniqueName, int32 TagId, const TCHAR* DisplayName, TOptional<int32> ParentTagId);

private:

	static constexpr int32 FNAME_INDEX_OFFSET = 512;

	static int32 GetFNameTagID(FName UniqueName)
	{
		return UniqueName.GetDisplayIndex().ToUnstableInt() + FNAME_INDEX_OFFSET;
	}

	struct FTagNameSetEntry
	{
		std::atomic_int32_t Data;

		int32 GetKey() const { return Data.load(std::memory_order_relaxed); }
		bool GetValue() const { return true; }
		bool IsEmpty() const { return Data.load(std::memory_order_relaxed) == 0; }			// NAME_None is treated as empty
		void SetKeyValue(int32 Key, bool Value) { Data.store(Key, std::memory_order_relaxed); }
		static uint32 KeyHash(int32 Key) { return static_cast<uint32>(Key); }
		static void ClearEntries(FTagNameSetEntry* Entries, int32 EntryCount)
		{
			memset(Entries, 0, EntryCount * sizeof(FTagNameSetEntry));
		}
	};
	typedef TGrowOnlyLockFreeHash<FTagNameSetEntry, int32, bool> FTagNameSet;

	FTagNameSet				AnnouncedNames;
	static FMalloc* 		Malloc;
};

FMalloc* FTagTrace::Malloc = nullptr;
static FTagTrace* GTagTrace = nullptr;

////////////////////////////////////////////////////////////////////////////////
FTagTrace::FTagTrace(FMalloc* InMalloc)
	: AnnouncedNames(InMalloc)
{
	Malloc = InMalloc;
	AnnouncedNames.Reserve(1024);
	AnnounceGenericTags();
	AnnounceSpecialTags();
	AnnounceLLMExtendedTags();
}

////////////////////////////////////////////////////////////////////////////////
void FTagTrace::AnnounceGenericTags() const
{
#if ENABLE_LOW_LEVEL_MEM_TRACKER
	#define TRACE_TAG_SPEC(Enum,Str,Stat,Group,ParentTag)\
	{\
		const uint32 DisplayLen = FCStringAnsi::Strlen(Str);\
		UE_TRACE_LOG(Memory, TagSpec, MemAllocChannel, DisplayLen * sizeof(ANSICHAR))\
			<< TagSpec.Tag((int32) ELLMTag::Enum)\
			<< TagSpec.Parent((int32) ParentTag)\
			<< TagSpec.Display(Str, DisplayLen);\
	}
	LLM_ENUM_GENERIC_TAGS(TRACE_TAG_SPEC);
	#undef TRACE_TAG_SPEC
#endif
}

////////////////////////////////////////////////////////////////////////////////
#if ENABLE_LOW_LEVEL_MEM_TRACKER
void FTagTrace::OnLLMInitialised(UPTRINT pThis)
{
	reinterpret_cast<FTagTrace*>(pThis)->AnnounceLLMExtendedTags();
}
#endif

////////////////////////////////////////////////////////////////////////////////
void FTagTrace::AnnounceLLMExtendedTags()
{
#if ENABLE_LOW_LEVEL_MEM_TRACKER
	// Our first call to AnnounceLLMExtendedTags can occur before FName's singleton has been constructed, and so
	// LLM reports that it is not yet fully initialised. We need to wait for LLM to have its Tags' FNames, so add
	// ourselves to its initialisation callback to reexecute this function at that point.
	FLowLevelMemTracker& Tracker = FLowLevelMemTracker::Get();
	if (!Tracker.IsInitialized())
	{
		UE::LLMPrivate::FPrivateCallbacks::AddInitialisedCallback(FTagTrace::OnLLMInitialised, reinterpret_cast<UPTRINT>(this));
		return;
	}

	// Announce all of the LLMTags that LLM currently knows about, and sign up to be notified when new ones are added.
	TArray<const UE::LLMPrivate::FTagData*> Tags = Tracker.GetTrackedTags();
	for (const UE::LLMPrivate::FTagData* Tag : Tags)
	{
		AnnounceLLMExtendedTag(Tracker, Tag);
	}
	UE::LLMPrivate::FPrivateCallbacks::AddTagCreationCallback(FTagTrace::OnAnnounceLLMExtendedTag, reinterpret_cast<UPTRINT>(this));
#endif
}

////////////////////////////////////////////////////////////////////////////////
#if ENABLE_LOW_LEVEL_MEM_TRACKER
void FTagTrace::OnAnnounceLLMExtendedTag(const UE::LLMPrivate::FTagData* LLMTag, UPTRINT pThis)
{
	FLowLevelMemTracker& Tracker = FLowLevelMemTracker::Get();
	reinterpret_cast<FTagTrace*>(pThis)->AnnounceLLMExtendedTag(Tracker, LLMTag);
}

////////////////////////////////////////////////////////////////////////////////
int32 FTagTrace::AnnounceLLMExtendedTag(FLowLevelMemTracker& Tracker, const UE::LLMPrivate::FTagData* LLMTag)
{
	bool bEnumTag = Tracker.GetTagIsEnumTag(LLMTag);
	ELLMTag EnumTag = ELLMTag::GenericTagCount;
	if (bEnumTag)
	{
		EnumTag = Tracker.GetTagClosestEnumTag(LLMTag);
		if (EnumTag < ELLMTag::GenericTagCount)
		{
			// We already announced this tag super-early as part of AnnounceGenericTags
			return static_cast<int32>(EnumTag);
		}
		// Not a generic tag, so this is a Platform or Project tag.
	}

	FName UniqueName = Tracker.GetTagUniqueName(LLMTag);
	int32 TagId = bEnumTag ? static_cast<int32>(EnumTag) : GetFNameTagID(UniqueName);

	if (!LLMTag->IsReportable())
	{
		return TagId;
	}

	int32 ParentTagId = -1;
	const UE::LLMPrivate::FTagData* ParentLLMTag = Tracker.GetTagParent(LLMTag);
	if (ParentLLMTag)
	{
		ParentTagId = AnnounceLLMExtendedTag(Tracker, ParentLLMTag);
	}

	TStringBuilder<FName::StringBufferSize+1> DisplayName;
	Tracker.GetTagDisplayPathName(LLMTag, DisplayName, FName::StringBufferSize);
	const TCHAR* DisplayNamePtr = *DisplayName;
	if (DisplayName.Len() >= FName::StringBufferSize)
	{
		DisplayNamePtr = nullptr; // Use the UniqueName instead
	}

	return AnnounceFNameTag(UniqueName, TagId, DisplayNamePtr, TOptional<int32>(ParentTagId));
}

#endif

////////////////////////////////////////////////////////////////////////////////
void FTagTrace::AnnounceSpecialTags() const
{
	auto EmitTag = [](const TCHAR* DisplayString, int32 Tag, int32 ParentTag)
	{
		const int32 DisplayLen = FCString::Strlen(DisplayString);
		UE_TRACE_LOG(Memory, TagSpec, MemAllocChannel, DisplayLen * sizeof(ANSICHAR))
			 << TagSpec.Tag(Tag)
			 << TagSpec.Parent(ParentTag)
			 << TagSpec.Display(DisplayString, DisplayLen);
	};

	EmitTag(TEXT("Trace"), TRACE_TAG, -1);

}

////////////////////////////////////////////////////////////////////////////////
int32 FTagTrace::AnnounceFNameTag(FName UniqueName, int32 TagId, const TCHAR* DisplayName, TOptional<int32> ParentTagId)
{
	if (TagId == -1)
	{
		TagId = GetFNameTagID(UniqueName);
	}

	// Find or add the item
	bool bAlreadyInTable;
	AnnouncedNames.FindOrAdd(TagId, true, &bAlreadyInTable);
	if (bAlreadyInTable)
	{
		return TagId;
	}

	// First time encountering this name, announce it
	ANSICHAR NameString[NAME_SIZE];
	int32 DisplayNameLen = DisplayName == nullptr ? 0 : FCString::Strlen(DisplayName);
	DisplayNameLen = DisplayNameLen + 1 > NAME_SIZE ? 0 : DisplayNameLen;
	if (DisplayNameLen > 0)
	{
		verify(FPlatformString::Convert(NameString, NAME_SIZE, DisplayName, DisplayNameLen + 1) != nullptr);
	}
	else
	{
		UniqueName.GetPlainANSIString(NameString);
	}

	if (!ParentTagId.IsSet())
	{
		ParentTagId = -1;

		if (DisplayNameLen > 0)
		{
			// By contract we pull the parent name from the UniqueName rather than the display string.
			// !ParentTagId.IsSet() does not currently occur when DisplayName is provided so we
			// handle this case with poor performance rather than adding stacksize or complexity to
			// handle it efficiently.
			UniqueName.GetPlainANSIString(NameString);
		}

		FAnsiStringView NameView(NameString);
		int32 LeafStart;
		if (NameView.FindLastChar('/', LeafStart))
		{
			FName ParentName(NameView.Left(LeafStart));
			ParentTagId = AnnounceFNameTag(ParentName, -1, nullptr, TOptional<int32>());
		}

		if (DisplayNameLen > 0)
		{
			verify(FPlatformString::Convert(NameString, NAME_SIZE, DisplayName, DisplayNameLen + 1) != nullptr);
		}
	}
	return AnnounceCustomTag(TagId, *ParentTagId, NameString);
}

////////////////////////////////////////////////////////////////////////////////
int32 FTagTrace::AnnounceCustomTag(int32 Tag, int32 ParentTag, const ANSICHAR* Display) const
{
	const uint32 DisplayLen = FCStringAnsi::Strlen(Display);
	UE_TRACE_LOG(Memory, TagSpec, MemAllocChannel, DisplayLen * sizeof(ANSICHAR))
		<< TagSpec.Tag(Tag)
		<< TagSpec.Parent(ParentTag)
		<< TagSpec.Display(Display, DisplayLen);
	return Tag;
}

#endif //UE_MEMORY_TAGS_TRACE_ENABLED


////////////////////////////////////////////////////////////////////////////////
void MemoryTrace_InitTags(FMalloc* InMalloc)
{
#if UE_MEMORY_TAGS_TRACE_ENABLED && UE_TRACE_ENABLED
	GTagTrace = (FTagTrace*)InMalloc->Malloc(sizeof(FTagTrace), alignof(FTagTrace));
	new (GTagTrace) FTagTrace(InMalloc);
#endif
}

////////////////////////////////////////////////////////////////////////////////
int32 MemoryTrace_AnnounceCustomTag(int32 Tag, int32 ParentTag, const TCHAR* Display)
{
#if UE_MEMORY_TAGS_TRACE_ENABLED && UE_TRACE_ENABLED
	//todo: How do we check if tag trace is active?
	if (GTagTrace)
	{
		return GTagTrace->AnnounceCustomTag(Tag, ParentTag, TCHAR_TO_ANSI(Display));
	}
#endif
	return -1;
}

////////////////////////////////////////////////////////////////////////////////
int32 MemoryTrace_AnnounceFNameTag(const FName& TagName)
{
#if UE_MEMORY_TAGS_TRACE_ENABLED && UE_TRACE_ENABLED
	if (GTagTrace)
	{
		return GTagTrace->AnnounceFNameTag(TagName, -1, nullptr, TOptional<int32>());
	}
#endif
	return -1;
}

////////////////////////////////////////////////////////////////////////////////
int32 MemoryTrace_GetActiveTag()
{
#if UE_MEMORY_TAGS_TRACE_ENABLED && UE_TRACE_ENABLED
	return GActiveTag;
#else
	return -1;
#endif
}

