// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProfilingDebugging/TraceAuxiliary.h"

#include "BuildSettings.h"
#include "Containers/AnsiString.h"
#include "ProfilingDebugging/CountersTrace.h"
#include "ProfilingDebugging/CsvProfilerConfig.h"
#include "ProfilingDebugging/StringsTrace.h"
#include "ProfilingDebugging/TagTrace.h"
#include "Trace/Trace.h"

////////////////////////////////////////////////////////////////////////////////
/// Configuration
////////////////////////////////////////////////////////////////////////////////

#if !defined(WITH_UNREAL_TRACE_COMMANDLINE_ARGS)
	#define UE_TRACEAUX_CMD_ARGS UE_TRACE_ENABLED
#else
	#define UE_TRACEAUX_CMD_ARGS WITH_UNREAL_TRACE_COMMANDLINE_ARGS
#endif

#if !defined(WITH_UNREAL_TRACE_CONSOLE_CMDS)
	#define UE_TRACEAUX_CON_CMDS UE_TRACE_ENABLED
#else
	#define  UE_TRACEAUX_CON_CMDS WITH_UNREAL_TRACE_CONSOLE_CMDS
#endif

#if !defined(WITH_UNREAL_TRACE_AUTOSTART)
	#define UE_TRACEAUX_AUTOSTART UE_TRACE_ENABLED
#else
	#define UE_TRACEAUX_AUTOSTART WITH_UNREAL_TRACE_AUTOSTART
#endif

#if !defined(UE_TRACEAUX_FULL)
	#define UE_TRACEAUX_FULL UE_TRACE_ENABLED
#endif

#if !defined(UE_TRACE_SERVER_LAUNCH_ENABLED)
	#define UE_TRACE_SERVER_LAUNCH_ENABLED (PLATFORM_DESKTOP && !UE_BUILD_SHIPPING && !IS_PROGRAM)
#endif

#if !defined(UE_TRACE_AUTOSTART)
	#define UE_TRACE_AUTOSTART 1
#endif

#if !defined(UE_TRACEAUX_EMIT_LLM_STATS)
	#define UE_TRACEAUX_EMIT_LLM_STATS (1 && ENABLE_LOW_LEVEL_MEM_TRACKER)
#endif

#if !defined(UE_TRACEAUX_EMIT_CSV_STATS)
	#define UE_TRACEAUX_EMIT_CSV_STATS (1 && CSV_PROFILER_STATS)
#endif

////////////////////////////////////////////////////////////////////////////////

#if UE_TRACE_SERVER_LAUNCH_ENABLED || UE_TRACE_SERVER_CONTROLS_ENABLED
#include "HAL/PlatformProcess.h"
#include "Misc/CommandLine.h"
#include "Misc/Parse.h"
#include "Misc/Paths.h"
#endif

#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"
#include <Windows.h>
#include "Windows/HideWindowsPlatformTypes.h"
#endif
#if PLATFORM_LINUX || PLATFORM_MAC
#include <sys/wait.h>
#include <semaphore.h>
#include <unistd.h>
#endif

#if UE_TRACE_MINIMAL_ENABLED

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include "CoreGlobals.h"
#include "HAL/FileManager.h"
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformTime.h"
#include "Logging/LogMacros.h"
#include "Misc/App.h"
#include "Misc/CString.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreDelegates.h"
#include "Misc/DateTime.h"
#include "Misc/Fork.h"
#include "Misc/Guid.h"
#include "Misc/PathViews.h"
#include "Misc/Paths.h"
#include "Misc/ScopeRWLock.h"
#include "Modules/ModuleManager.h"
#include "ProfilingDebugging/CallstackTrace.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "ProfilingDebugging/PlatformEvents.h"
#include "String/ParseTokens.h"
#include "Templates/Invoke.h"
#include "Templates/UnrealTemplate.h"
#include "Trace/Trace.inl"

////////////////////////////////////////////////////////////////////////////////
/// Stats and logging
////////////////////////////////////////////////////////////////////////////////

CSV_DEFINE_CATEGORY(Trace, true);

#if UE_TRACEAUX_EMIT_LLM_STATS
LLM_DEFINE_TAG(TraceLog);
LLM_DEFINE_TAG(TraceLog_BlockPool);
LLM_DEFINE_TAG(TraceLog_FixedBuffers);
LLM_DEFINE_TAG(TraceLog_SharedBuffers);
LLM_DEFINE_TAG(TraceLog_Cache);
#endif

#if NO_LOGGING
FTraceAuxiliary::FLogCategoryAlias LogTrace;
#else
DEFINE_LOG_CATEGORY_STATIC(LogTrace, Log, All);
#endif

////////////////////////////////////////////////////////////////////////////////
/// Types and constants
////////////////////////////////////////////////////////////////////////////////

enum class EWorkerThreadConfig : uint8 {
	Unset,
	OnDemand,		// Worker thread started when first trace is started
	OnInit,			// Worker thread started on boot
	Never,			// Worker threda never started (rely on end frame pump)
};

////////////////////////////////////////////////////////////////////////////////
const FTraceAuxiliary::FChannelPreset GDefaultChannels(TEXT("Default"), TEXT("cpu,gpu,frame,log,bookmark,screenshot,region"), false);
const FTraceAuxiliary::FChannelPreset GMemoryChannels(TEXT("Memory"), TEXT("memtag,memalloc,callstack,module"), true);
const FTraceAuxiliary::FChannelPreset GMemoryLightChannels(TEXT("Memory_Light"), TEXT("memtag,memalloc"), true);

////////////////////////////////////////////////////////////////////////////////
/**
 * FTraceAuxiliaryImpl is an implementation of the FTraceAuxiliary public interface.
 * It is also remembers and manages the state of TraceLog when it comes to:
 * * Channels
 * * Connections
 * * Command line arguments for forking
 * * Misc state (like worker thread)
 */
class FTraceAuxiliaryImpl
{
public:
	FString GetDest() const;
	bool IsConnected() const;
	bool IsConnected(FGuid& OutSessionGuid, FGuid& OutTraceGuid);
	FTraceAuxiliary::EConnectionType GetConnectionType() const;
	void GetActiveChannelsString(FStringBuilderBase& String) const;
	void EnableChannels(const TCHAR* ChannelList, const FTraceAuxiliary::FLogCategoryAlias& LogCategory);
	void EnableChannels(TConstArrayView<uint32> ChannelIds, TMap<uint32, FString>* OutReason);
	void DisableChannels(const TCHAR* ChannelList, const FTraceAuxiliary::FLogCategoryAlias& LogCategory);
	void DisableChannels(TConstArrayView<uint32> ChannelIds, TMap<uint32, FString>* OutReason);
	bool Connect(FTraceAuxiliary::EConnectionType Type, const TCHAR* Parameter, const FTraceAuxiliary::FLogCategoryAlias& LogCategory, uint16 SendFlags);
	bool ConnectRelay(UPTRINT Handle, UE::Trace::IoWriteFunc WriteFunc, UE::Trace::IoCloseFunc CloseFunc, uint16 SendFlags);
#if UE_TRACE_ALLOW_SECURE_TRACING
	bool InitSecure();
	bool ConnectSecure(FTraceAuxiliary::FSecureOptions& Options, uint16 SendFlags);
	FString GetSecureHost() const;
	uint16 GetSecurePort() const;
#endif
	bool Stop(const FTraceAuxiliary::FLogCategoryAlias& LogCategory);
	void OnClosed();
	void FreezeReadOnlyChannels();
	void ResumeChannels();
	void PauseChannels();
	bool IsPaused();
	void EnableCommandlineChannels();
	void EnableCommandlineChannelsPostInitialize();
	bool HasCommandlineChannels() const;
	void SetTruncateFile(bool bTruncateFile);
	void StartWorkerThread();
	void StartEndFrameUpdate();
	void RegisterEndFrameCallbacks();
	bool WriteSnapshot(const TCHAR* InFilePath, const FTraceAuxiliary::FLogCategoryAlias& LogCategory, uint32 MaxSize);
	bool SendSnapshot(const TCHAR* InHost, uint32 InPort, const FTraceAuxiliary::FLogCategoryAlias& LogCategory, uint32 MaxSize);
	void AddCommandlineChannels(const TCHAR* ChannelList);
	void ResetCommandlineChannels();
	void UpdateStats();
	void EmitStats(const UE::Trace::FStatistics& Stats);
	void SetHasPanicked() { bHasPanicked = true; };
	bool GetHasPanicked() { return bHasPanicked; };

	// True if this is parent process with forking requested before forking.
	bool IsParentProcessAndPreFork();

private:
	enum class EState : uint8
	{
		Stopped,
		Tracing,
	};

	struct FChannelEntry
	{
		FString Name;
		bool bActive = false;
	};

	template<typename T, typename... Ts> void ForEachChannel(const TCHAR* ChannelList, bool bResolvePresets, T&& Callable, Ts&&... Args);
	static uint32 HashChannelName(const TCHAR* Name);
	bool SendToHost(const TCHAR* Host, const FTraceAuxiliary::FLogCategoryAlias& LogCategory, uint16 SendFlags);
	bool WriteToFile(const TCHAR* Path, const FTraceAuxiliary::FLogCategoryAlias& LogCategory, uint16 SendFlags);
	bool FinalizeFilePath(const TCHAR* InPath, FString& OutPath, const FTraceAuxiliary::FLogCategoryAlias& LogCategory);
	void AddCommandlineChannel(const TCHAR* Name, const FTraceAuxiliary::FLogCategoryAlias& LogCategory);
	void EmitDiagnostics(const UE::Trace::FStatistics& Stats);
	void OnGetOnScreenMessages(FCoreDelegates::FSeverityMessageMap& OutMessages);
	static void FrameBasedUpdate();

	template<typename ChannelType>
	bool EnableChannel(ChannelType Channel, const FTraceAuxiliary::FLogCategoryAlias& LogCategory, FString* OutReason = nullptr);
	template<typename ChannelType>
	bool DisableChannel(ChannelType Channel, const FTraceAuxiliary::FLogCategoryAlias& LogCategory, FString* OutReason = nullptr);

	typedef TMap<uint32, FChannelEntry, TInlineSetAllocator<128>> ChannelSet;
	ChannelSet CommandlineChannels;
	bool bWorkerThreadStarted = false;
	bool bTruncateFile = false;
	bool bReadOnlyChannelsFrozen = false;
	bool bHasPanicked = false;
	FString PausedPreset;
	FDelegateHandle EndFrameUpdateHandle;
	FDelegateHandle OnGetOnScreenMessageHandle;
	FRWLock DiagnosticsLock;
	FString DiagnosticMessage;

#if UE_TRACE_ALLOW_SECURE_TRACING
	mutable FRWLock SecureAuthLock;
	TArray<uint8> SecureToken;
	bool bSecureInit = false;
	FAnsiString SecureIdentity;
	FString SecureHost;
	uint16 SecurePort = 0;
#endif

	struct FCurrentTraceTarget
	{
		FString TraceDest;
		FTraceAuxiliary::EConnectionType TraceType = FTraceAuxiliary::EConnectionType::None;
	} CurrentTraceTarget;
	mutable FRWLock CurrentTargetLock;
};

////////////////////////////////////////////////////////////////////////////////
/// Statics
////////////////////////////////////////////////////////////////////////////////

static FTraceAuxiliaryImpl GTraceAuxiliary;
static const TCHAR* GTraceConfigSection = TEXT("Trace.Config");
static UE::Trace::FInitializeDesc GInitializeDesc;
static EWorkerThreadConfig GWorkerThreadConfig = EWorkerThreadConfig::Unset;
static FDelegateHandle GEndFrameDelegateHandle;
static FDelegateHandle GOnPostForkHandle;
static bool GTraceFileTimestamps = false;

#if UE_TRACEAUX_AUTOSTART
// Whether to start tracing automatically at start or wait to initiate via Console Command.
// This value can also be set by passing '-traceautostart=[0|1]' on command line.
static bool GTraceAutoStart = UE_TRACE_AUTOSTART ? true : false;
#else
constexpr bool GTraceAutoStart = false;
#endif

////////////////////////////////////////////////////////////////////////////////
bool FTraceAuxiliaryImpl::IsParentProcessAndPreFork()
{
	return FForkProcessHelper::IsForkRequested() && !FForkProcessHelper::IsForkedChildProcess();
}

////////////////////////////////////////////////////////////////////////////////
void FTraceAuxiliaryImpl::AddCommandlineChannel(const TCHAR* Name, const FTraceAuxiliary::FLogCategoryAlias& LogCategory)
{
#if UE_TRACEAUX_CMD_ARGS
	uint32 Hash = HashChannelName(Name);

	if (CommandlineChannels.Find(Hash) != nullptr)
	{
		return;
	}

	FChannelEntry& Value = CommandlineChannels.Add(Hash, {});
	Value.Name = Name;

	if (IsConnected() && !Value.bActive)
	{
		Value.bActive = EnableChannel(*Value.Name, LogCategory);
	}
#endif //UE_TRACEAUX_CMD_ARGS
}

////////////////////////////////////////////////////////////////////////////////
void FTraceAuxiliaryImpl::AddCommandlineChannels(const TCHAR* ChannelList)
{
#if UE_TRACEAUX_CMD_ARGS
	ForEachChannel(ChannelList, true, &FTraceAuxiliaryImpl::AddCommandlineChannel, LogTrace);
#endif //UE_TRACEAUX_CMD_ARGS
}

////////////////////////////////////////////////////////////////////////////////
void FTraceAuxiliaryImpl::ResetCommandlineChannels()
{
#if UE_TRACEAUX_CMD_ARGS
	CommandlineChannels.Reset();
#endif //UE_TRACEAUX_CMD_ARGS
}

////////////////////////////////////////////////////////////////////////////////
void FTraceAuxiliaryImpl::EnableChannels(const TCHAR* ChannelList, const FTraceAuxiliary::FLogCategoryAlias& LogCategory)
{
	if (ChannelList)
	{
		ForEachChannel(ChannelList, true, &FTraceAuxiliaryImpl::EnableChannel<const TCHAR*>, LogCategory, nullptr);
	}
}

////////////////////////////////////////////////////////////////////////////////
void FTraceAuxiliaryImpl::EnableChannels(TConstArrayView<uint32> ChannelIds, TMap<uint32, FString>* OutReason)
{
	for (const auto ChannelId : ChannelIds)
	{
		FString DenyReason;
		EnableChannel(ChannelId, LogTrace, &DenyReason);
		if (OutReason && !DenyReason.IsEmpty())
		{
			OutReason->Add(ChannelId, DenyReason);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////
void FTraceAuxiliaryImpl::DisableChannels(const TCHAR* ChannelList, const FTraceAuxiliary::FLogCategoryAlias& LogCategory)
{
	if (ChannelList)
	{
		ForEachChannel(ChannelList, true, &FTraceAuxiliaryImpl::DisableChannel<const TCHAR*>, LogCategory, nullptr);
	}
	else
	{
		// Disable all channels.
		TStringBuilder<128> EnabledChannels;
		GetActiveChannelsString(EnabledChannels);
		ForEachChannel(EnabledChannels.ToString(), true, &FTraceAuxiliaryImpl::DisableChannel<const TCHAR*>, LogCategory, nullptr);
	}
}

////////////////////////////////////////////////////////////////////////////////
void FTraceAuxiliaryImpl::DisableChannels(TConstArrayView<uint32> ChannelIds, TMap<uint32, FString>* OutReason)
{
	for (const auto ChannelId : ChannelIds)
	{
		FString DenyReason;
		DisableChannel(ChannelId, LogTrace, &DenyReason);
		if (OutReason && !DenyReason.IsEmpty())
		{
			OutReason->Add(ChannelId, DenyReason);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////
/**
 * Parses channel tokens from a "chan1, chan2, ...." string and expands presets
 * to an output array.
 * @param ChannelList Input string
 * @param bResolvePreset If presets should be resolved further
 * @param OutTokens Output tokens. Lifetime of view are the same as the input views.
 */
void ParseTokens(FStringView ChannelList, bool bResolvePresets, TArray<FStringView>& OutTokens, bool bToOwnedStrings, TArray<FString>& OutOwnedTokens)
{
	UE::String::ParseTokens(ChannelList, TEXT(","), [&] (const FStringView& Token)
	{
		TCHAR Name[80];
		const size_t ChannelNameSize = Token.CopyString(Name, UE_ARRAY_COUNT(Name) - 1);
		Name[ChannelNameSize] = '\0';

		if (bResolvePresets)
		{
			FString ConfigChannelPreset;
			// Check against hard coded presets
			if (FCString::Stricmp(Name, GDefaultChannels.Name) == 0)
			{
				ParseTokens(GDefaultChannels.ChannelList, false, OutTokens, false, OutOwnedTokens);
				return;
			}
			else if (FCString::Stricmp(Name,GMemoryChannels.Name) == 0)
			{
				ParseTokens(GMemoryChannels.ChannelList, false, OutTokens, false, OutOwnedTokens);
				return;
			}
			else if (FCString::Stricmp(Name, GMemoryLightChannels.Name) == 0)
			{
				ParseTokens(GMemoryLightChannels.ChannelList, false, OutTokens, false, OutOwnedTokens);
				return;
			}
			// Check against data driven presets (if available)
			else if (GConfig && GConfig->GetString(TEXT("Trace.ChannelPresets"), Name, ConfigChannelPreset, GEngineIni))
			{
				// Since ConfigChannelPreset is temporary string we need to output
				// those to the owned tokens
				ParseTokens(*ConfigChannelPreset, false, OutTokens, true, OutOwnedTokens);
				return;
			}
		}
		bToOwnedStrings ? OutOwnedTokens.Emplace(Token) : OutTokens.Add(Token);
	});
}

////////////////////////////////////////////////////////////////////////////////
template<typename T, typename... Ts>
void FTraceAuxiliaryImpl::ForEachChannel(const TCHAR* ChannelList, bool bResolvePresets, T&& Callable, Ts&&... Args)
{
	check(ChannelList);
	TArray<FStringView> ExpandedChannels;
	TArray<FString> ExpandedChannelStrings;
	ParseTokens(ChannelList, bResolvePresets, ExpandedChannels, false, ExpandedChannelStrings);
	for (const FStringView& Channel : ExpandedChannels)
	{
		TCHAR Name[80];
		const size_t ChannelNameSize = Channel.TrimStartAndEnd().CopyString(Name, UE_ARRAY_COUNT(Name) - 1);
		Name[ChannelNameSize] = '\0';

		Invoke(Callable, this, Name, std::forward<Ts>(Args)...);
	}
	for (const FString& Channel : ExpandedChannelStrings)
	{
		Invoke(Callable, this, *Channel, std::forward<Ts>(Args)...);
	}
}

////////////////////////////////////////////////////////////////////////////////
template <typename ChannelType>
bool FTraceAuxiliaryImpl::EnableChannel(ChannelType ChannelIdentifier, const FTraceAuxiliary::FLogCategoryAlias& LogCategory, FString* OutReason)
{
	// Channel names have been provided by the user and may not exist yet. As
	// we want to maintain bActive accurately (channels toggles are reference
	// counted), we will first check Trace knows of the channel.
	UE::Trace::FChannel* Channel = UE::Trace::FindChannel(ChannelIdentifier);
	if (!Channel)
	{
		if (OutReason)
		{
			FString ChannelStr = LexToString(ChannelIdentifier);
			OutReason->Appendf(TEXT("Unknown channel %s"), *ChannelStr);
		}
		return false;
	}

	// Build an wide representation of the name for logging and platform
	// events purposes.
	// todo: This can be moved into the log scope once platform events is being triggered by channel callbacks
	TStringBuilder<64> ChannelName;
	{
		const ANSICHAR* ChannelNameA = nullptr;
		const uint32 ChannelNameLen = Channel->GetName(&ChannelNameA);
		ChannelName << FAnsiStringView(ChannelNameA, ChannelNameLen);
	}

	// It is not possible to change read only channels once trace is initialized.
	if (bReadOnlyChannelsFrozen && Channel->IsReadOnly())
	{
		if (!Channel->IsEnabled())
		{
			UE_LOG_REF(
				LogCategory,
				Error,
				TEXT("Channel '%s' is read only. It is not allowed to manually enable this channel."),
				ChannelName.ToString()
			);
		}
		return Channel->IsEnabled();
	}

	const TCHAR* DenyReason = nullptr;
	const bool bIsEnabled = Channel->Toggle(true, &DenyReason);
	if (DenyReason)
	{
		if (OutReason)
		{
			OutReason->Append(DenyReason);
		}
		else
		{
			UE_LOG_REF(
				LogCategory,
				Warning,
				TEXT("Cannot enable channel '%s': %s"),
				ChannelName.ToString(),
				DenyReason
			);
		}
	}

	UE_CLOGF(bIsEnabled, LogTrace, Verbose, "Enabled channel '%ls'.", ChannelName.ToString());
	return bIsEnabled;
}

////////////////////////////////////////////////////////////////////////////////
template <typename ChannelType>
bool FTraceAuxiliaryImpl::DisableChannel(ChannelType ChannelIdentifier, const FTraceAuxiliary::FLogCategoryAlias& LogCategory, FString* OutReason)
{
	// Channel names have been provided by the user and may not exist yet. As
	// we want to maintain bActive accurately we will first check Trace knows of the channel.
	UE::Trace::FChannel* Channel = UE::Trace::FindChannel(ChannelIdentifier);
	if (!Channel)
	{
		if (OutReason)
		{
			FString ChannelStr = LexToString(ChannelIdentifier);
			OutReason->Appendf(TEXT("Unknown channel %s"), *ChannelStr);
		}
		return false;
	}

	// Build an wide representation of the name for logging and platform
	// events purposes.
	// todo: This can be moved into the log scope once platform events is being triggered by channel callbacks
	TStringBuilder<64> ChannelName;
	{
		const ANSICHAR* ChannelNameA = nullptr;
		const uint32 ChannelNameLen = Channel->GetName(&ChannelNameA);
		ChannelName << FAnsiStringView(ChannelNameA, ChannelNameLen);
	}

	// It is not possible to change read only channels once trace is initialized.
	if (bReadOnlyChannelsFrozen && Channel->IsReadOnly())
	{
		UE_LOG_REF(
			LogCategory,
			Error,
			TEXT("Channel '%s' is read only. It is not allowed to manually disable this channel."),
			ChannelName.ToString()
		);
		return Channel->IsEnabled();
	}

	const TCHAR* DenyReason = nullptr;
	const bool bIsEnabled = Channel->Toggle(false, &DenyReason);
	if (DenyReason)
	{
		if (OutReason)
		{
			OutReason->Append(DenyReason);
		}
		else
		{
			UE_LOG_REF(
				LogCategory,
				Warning,
				TEXT("Cannot disable channel '%s': %s"),
				ChannelName.ToString(), DenyReason
			);
		}
	}

	UE_CLOGF(!bIsEnabled, LogTrace, Verbose, "Disabled channel '%ls'.", ChannelName.ToString());
	return bIsEnabled;
}

////////////////////////////////////////////////////////////////////////////////
uint32 FTraceAuxiliaryImpl::HashChannelName(const TCHAR* Name)
{
	uint32 Hash = 5381;
	for (const TCHAR* c = Name; *c; ++c)
	{
		uint32 LowerC = *c | 0x20;
		Hash = ((Hash << 5) + Hash) + LowerC;
	}
	return Hash;
}

////////////////////////////////////////////////////////////////////////////////
bool FTraceAuxiliaryImpl::Connect(FTraceAuxiliary::EConnectionType Type, const TCHAR* Parameter, const FTraceAuxiliary::FLogCategoryAlias& LogCategory, uint16 SendFlags)
{
	if (bHasPanicked)
	{
		UE_LOG_REF(LogCategory, Warning, TEXT("Trace has panicked, cannot start new trace."));
		return false;
	}
	// Connect to trace server or write to a file, but only if we're not already tracing.
	if (UE::Trace::IsTracing())
	{
		UE_LOG_REF(LogCategory, Warning, TEXT("Already tracing!"));
		return true;
	}

	bool bConnected = false;
	if (Type == FTraceAuxiliary::EConnectionType::Network)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FTraceAux_SendToHost);
		bConnected = SendToHost(Parameter, LogCategory, SendFlags);
		if (bConnected)
		{
			UE_LOG_REF(LogCategory, Display, TEXT("Trace started (connected to trace server %s)."), *GetDest());
		}
		else
		{
			UE_LOG_REF(LogCategory, Error, TEXT("Trace failed to connect (trace host: %s)!"), Parameter ? Parameter : TEXT(""));
		}
	}
	else if (Type == FTraceAuxiliary::EConnectionType::File)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FTraceAux_WriteToFile);
		bConnected = WriteToFile(Parameter, LogCategory, SendFlags);
		if (bConnected)
		{
			UE_LOG_REF(LogCategory, Display, TEXT("Trace started (writing to file \"%s\")."), *GetDest());
		}
		else
		{
			UE_LOG_REF(LogCategory, Error, TEXT("Trace failed to connect (file: \"%s\")!"), Parameter ? Parameter : TEXT(""));
		}
	}
	else if (Type == FTraceAuxiliary::EConnectionType::None)
	{
		UE_LOG_REF(LogCategory, Log, TEXT("No trace connection. Tracing to in-memory cache."));
	}
	else
	{
		UE_LOG_REF(LogCategory, Error, TEXT("Unknown trace connection type (%u)!"), uint32(Type));
	}

	if (bConnected)
	{
		FTraceAuxiliary::EConnectionType StartedType = FTraceAuxiliary::EConnectionType::None;
		FString StartedDest;

		{
			FReadScopeLock _(CurrentTargetLock);
			StartedType = CurrentTraceTarget.TraceType;
			StartedDest = CurrentTraceTarget.TraceDest;
		}

		TRACE_CPUPROFILER_EVENT_SCOPE(FTraceAux_OnTraceStarted);
		FTraceAuxiliary::OnTraceStarted.Broadcast(StartedType, StartedDest);
	}

	return bConnected;
}

////////////////////////////////////////////////////////////////////////////////
bool FTraceAuxiliaryImpl::ConnectRelay(UPTRINT Handle, UE::Trace::IoWriteFunc WriteFunc, UE::Trace::IoCloseFunc CloseFunc, uint16 SendFlags)
{
	check(WriteFunc && CloseFunc);
	if (bHasPanicked)
	{
		UE_LOGF(LogTrace, Warning, "Trace has panicked, cannot start new trace.");
		return false;
	}

	if (UE::Trace::IsTracing())
	{
		UE_LOGF(LogTrace, Warning, "Already tracing!");
		return true;
	}

	const bool bConnected = UE::Trace::RelayTo(Handle, WriteFunc, CloseFunc, SendFlags);
	if (bConnected)
	{
		UE_LOGF(LogTrace, Display, "Trace started (connected to relay endpoint).");

		FTraceAuxiliary::EConnectionType StartedType = FTraceAuxiliary::EConnectionType::Relay;
		FString StartedDest = FString::Printf(TEXT("Relay endpoint (Handle: 0x%llx)"), Handle);

		{
			FWriteScopeLock _(CurrentTargetLock);
			CurrentTraceTarget.TraceType = StartedType;
			CurrentTraceTarget.TraceDest = StartedDest;
		}

		TRACE_CPUPROFILER_EVENT_SCOPE(FTraceAux_OnTraceStarted);
		FTraceAuxiliary::OnTraceStarted.Broadcast(StartedType, StartedDest);
	}
	else
	{
		UE_LOGF(LogTrace, Error, "Trace failed to connect relay endpoint!");
	}
	return bConnected;
}

////////////////////////////////////////////////////////////////////////////////
#if UE_TRACE_ALLOW_SECURE_TRACING 

////////////////////////////////////////////////////////////////////////////////
bool FTraceAuxiliaryImpl::InitSecure()
{
	if (!bSecureInit)
	{
		FString CAPath, PrivateKeyPath;
		int32 SecurePortInt32 = 0;
		bool bUsePreSharedKeys = false;
		if (GConfig)
		{
			GConfig->GetString(TEXT("Trace"), TEXT("SecureCA"), CAPath, GEngineIni);
			GConfig->GetString(TEXT("Trace"), TEXT("SecureKey"), PrivateKeyPath, GEngineIni);
			GConfig->GetBool(TEXT("Trace"), TEXT("SecureUsePreSharedKeys"), bUsePreSharedKeys, GEngineIni);
			GConfig->GetInt(TEXT("Trace"), TEXT("SecurePort"), SecurePortInt32, GEngineIni);
		}

		FParse::Value(FCommandLine::Get(), TEXT("-TraceSecureCA="), CAPath);
		FParse::Value(FCommandLine::Get(), TEXT("-TraceSecureKey="), PrivateKeyPath);
		FParse::Value(FCommandLine::Get(), TEXT("-TraceSecureHost="), SecureHost);
		FParse::Value(FCommandLine::Get(), TEXT("-TraceSecurePort="), SecurePortInt32);
		bUsePreSharedKeys |= FParse::Param(FCommandLine::Get(), TEXT("-TraceSecureUsePreSharedKeys"));
		FAnsiString CAPathAnsi(CAPath);
		FAnsiString PrivateKeyPathAnsi(PrivateKeyPath);

		UE_LOGF(LogTrace, Verbose, "Initializing secure trace.");
		UE::Trace::FSecureTraceSettings Settings;

		// Setup how to connect to outside peers
		if (SecureHost.IsEmpty())
		{
			UE_LOGF(LogTrace, Display, "No secure host specified, public IP will be used.");
		}
		if (SecurePortInt32 > 0 && SecurePortInt32 < UINT16_MAX)
		{
			Settings.Port = SecurePort = uint16(SecurePortInt32);
		}
		else
		{
			UE_LOGF(LogTrace, Display, "No secure port specified, system will assign port.");
			Settings.Port = 0;
		}

		// Determine which security mode we will use.
		if (!CAPathAnsi.IsEmpty() && !PrivateKeyPathAnsi.IsEmpty())
		{
			//todo: Verify files exists and are readable
			Settings.Mode = UE::Trace::ESecureTraceMode::CertificateFile;
			UE_LOGF(LogTrace, Verbose, "Using certificate '%s' and key '%s'", *CAPathAnsi, *PrivateKeyPathAnsi);
			Settings.CertificatePath = *CAPathAnsi;
			Settings.PrivateKeyPath = *PrivateKeyPathAnsi;
		}
		else if (bUsePreSharedKeys)
		{
			Settings.Mode = UE::Trace::ESecureTraceMode::PreSharedKeys;
			UE_LOGF(LogTrace, Verbose, "Secure trace mode: Using pre shared keys");
		}
		else
		{
			Settings.Mode = UE::Trace::ESecureTraceMode::SelfSigned;
			UE_LOGF(LogTrace, Verbose, "Secure trace mode: Using self signed certificate");
		}

		if (!UE::Trace::SecureInitialize(Settings))
		{
			UE_LOGF(LogTrace, Error, "Secure trace initialization failed.");
			return false;
		}
		bSecureInit = true;
	}
	return true;
}

////////////////////////////////////////////////////////////////////////////////
bool FTraceAuxiliaryImpl::ConnectSecure(FTraceAuxiliary::FSecureOptions& InOptions, uint16 SendFlags)
{
	if (bHasPanicked)
	{
		UE_LOGF(LogTrace, Warning, "Trace has panicked, cannot start new trace.");
		return false;
	}
	UE::Trace::FSecureTraceOptions SecureOptions;
	if (!InOptions.AuthorizationToken.IsEmpty())
	{
		// We're okay to write this before the actual connection
		// is created here since we don't allow overlapping ongoing
		// connections with a new pending connection.
		FWriteScopeLock _(SecureAuthLock);
		if (!InitSecure())
		{
			return false;
		}
		SecureToken = InOptions.AuthorizationToken;
		SecureIdentity = InOptions.Identity;
		SecureOptions.AuthorizationTokenSize = SecureToken.Num();
		SecureOptions.AuthorizationToken = SecureToken.GetData();
		SecureOptions.Identity = *SecureIdentity;
	}
	else
	{
		SecureOptions.AuthorizationTokenSize = 0u;
		SecureOptions.AuthorizationToken = nullptr;
		SecureOptions.Identity = nullptr;
	}
	SecureOptions.TimeoutMs = uint32(InOptions.TimeoutSeconds * 1000);

	const bool bSuccess = UE::Trace::SecureSendTo(SecureOptions, SendFlags);
	if (bSuccess)
	{
		FWriteScopeLock _(CurrentTargetLock);
		CurrentTraceTarget.TraceType = FTraceAuxiliary::EConnectionType::SecureNetwork;
		CurrentTraceTarget.TraceDest = TEXT("Secure network trace");
	}

	return bSuccess;
}

////////////////////////////////////////////////////////////////////////////////
FString FTraceAuxiliaryImpl::GetSecureHost() const
{
	FWriteScopeLock _(SecureAuthLock);
	return SecureHost;
}

////////////////////////////////////////////////////////////////////////////////
uint16 FTraceAuxiliaryImpl::GetSecurePort() const
{
	FWriteScopeLock _(SecureAuthLock);
	return SecurePort;
}

#endif // UE_TRACE_ALLOW_SECURE_TRACING

////////////////////////////////////////////////////////////////////////////////
bool FTraceAuxiliaryImpl::Stop(const FTraceAuxiliary::FLogCategoryAlias& LogCategory)
{
	if (bHasPanicked)
	{
		UE_LOG_REF(LogCategory, Warning, TEXT("Trace has panicked, cannot stop trace."));
		return false;
	}
	if (IsParentProcessAndPreFork())
	{
		return false;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FTraceAux_Stop);

	if (!UE::Trace::Stop())
	{
		return false;
	}

	FString StoppedDest;
	FTraceAuxiliary::EConnectionType StoppedType = FTraceAuxiliary::EConnectionType::None;
	PausedPreset.Empty();

	{
		FWriteScopeLock _(CurrentTargetLock);
		StoppedType = CurrentTraceTarget.TraceType;
		CurrentTraceTarget.TraceType = FTraceAuxiliary::EConnectionType::None;
		StoppedDest = CurrentTraceTarget.TraceDest;
		CurrentTraceTarget.TraceDest.Reset();
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FTraceAux_OnTraceStopped);
	FTraceAuxiliary::OnTraceStopped.Broadcast(StoppedType, StoppedDest);

	return true;
}

////////////////////////////////////////////////////////////////////////////////
void FTraceAuxiliaryImpl::OnClosed()
{
	// Clear the current target values
	{
		FWriteScopeLock _(CurrentTargetLock);
		CurrentTraceTarget.TraceType = FTraceAuxiliary::EConnectionType::None;
		CurrentTraceTarget.TraceDest.Reset();
	}
#if UE_TRACE_ALLOW_SECURE_TRACING
	{
		FWriteScopeLock _(SecureAuthLock);
		SecureToken.Reset(0);
	}
#endif
}

////////////////////////////////////////////////////////////////////////////////
void FTraceAuxiliaryImpl::FreezeReadOnlyChannels()
{
	bReadOnlyChannelsFrozen = true;
}

////////////////////////////////////////////////////////////////////////////////
void FTraceAuxiliaryImpl::ResumeChannels()
{
	// Enable channels from the "paused" preset.
	ForEachChannel(*PausedPreset, false, &FTraceAuxiliaryImpl::EnableChannel<const TCHAR*>, LogTrace, nullptr);

	PausedPreset.Empty();
}

////////////////////////////////////////////////////////////////////////////////
void FTraceAuxiliaryImpl::PauseChannels()
{
	TStringBuilder<128> EnabledChannels;
	GetActiveChannelsString(EnabledChannels);

	// Save the list of enabled channels as the current "paused" preset.
	// The "paused" preset can only be used in the Trace.Resume command / API.
	PausedPreset = EnabledChannels.ToString();

	// Disable all "paused" channels.
	ForEachChannel(*PausedPreset, true, &FTraceAuxiliaryImpl::DisableChannel<const TCHAR*>, LogTrace, nullptr);
}

////////////////////////////////////////////////////////////////////////////////
bool FTraceAuxiliaryImpl::IsPaused()
{
	return !PausedPreset.IsEmpty();
}

////////////////////////////////////////////////////////////////////////////////
void FTraceAuxiliaryImpl::EnableCommandlineChannels()
{
#if UE_TRACEAUX_CMD_ARGS
	if (IsParentProcessAndPreFork())
	{
		return;
	}

	for (auto& ChannelPair : CommandlineChannels)
	{
		if (!ChannelPair.Value.bActive)
		{
			ChannelPair.Value.bActive = EnableChannel(*ChannelPair.Value.Name, LogTrace);
		}
	}
#endif
}

////////////////////////////////////////////////////////////////////////////////
void FTraceAuxiliaryImpl::EnableCommandlineChannelsPostInitialize()
{
#if UE_TRACEAUX_CMD_ARGS
	for (auto& ChannelPair : CommandlineChannels)
	{
		// Intentionally enable channel without checking current state.
		ChannelPair.Value.bActive = EnableChannel(*ChannelPair.Value.Name, LogTrace);
	}
#endif
}

////////////////////////////////////////////////////////////////////////////////
bool FTraceAuxiliaryImpl::HasCommandlineChannels() const
{
#if UE_TRACEAUX_CMD_ARGS
	return !CommandlineChannels.IsEmpty();
#else
	return false;
#endif
}

////////////////////////////////////////////////////////////////////////////////
void FTraceAuxiliaryImpl::SetTruncateFile(bool bNewTruncateFileState)
{
	bTruncateFile = bNewTruncateFileState;
}

////////////////////////////////////////////////////////////////////////////////
bool FTraceAuxiliaryImpl::SendToHost(const TCHAR* InHost, const FTraceAuxiliary::FLogCategoryAlias& LogCategory, uint16 SendFlags)
{
	FStringView HostView(InHost);
	// Parse port if specified. Default is "0" which indicates default port will be used.
	uint32 Port = 0;
	int32 Separator = INDEX_NONE;
	if (HostView.FindChar(TEXT(':'), Separator))
	{
		LexFromString(Port, HostView.RightChop(Separator + 1).GetData());
		HostView.LeftInline(Separator);
	}

	FString Host(HostView);
	if (!UE::Trace::SendTo(*Host, Port, SendFlags))
	{
		UE_LOG_REF(LogCategory, Warning, TEXT("Unable to trace to host '%s'"), InHost);
		return false;
	}

	{
		FWriteScopeLock _(CurrentTargetLock);
		CurrentTraceTarget.TraceType = FTraceAuxiliary::EConnectionType::Network;
		CurrentTraceTarget.TraceDest = MoveTemp(InHost);
	}
	return true;
}

////////////////////////////////////////////////////////////////////////////////
bool FTraceAuxiliaryImpl::FinalizeFilePath(const TCHAR* InPath, FString& OutPath, const FTraceAuxiliary::FLogCategoryAlias& LogCategory)
{
	const FStringView Path(InPath);

	// Default file name functor
	auto GetDefaultName = []
		{
			const FDateTime Now = FDateTime::Now();
#if PLATFORM_LINUX
			return Now.ToString(TEXT("%Y%m%d_%H%M%S_")).Appendf(TEXT("%06" INT64_X_FMT "_%06d.utrace"), (Now.GetTicks() % 10000000LL), FPlatformProcess::GetCurrentProcessId());
#else
			return Now.ToString(TEXT("%Y%m%d_%H%M%S_")).Appendf(TEXT("%06" INT64_X_FMT ".utrace"), (Now.GetTicks() % 10000000LL));
#endif
		};

	if (Path.IsEmpty())
	{
		const FString Name = GetDefaultName();
		return FinalizeFilePath(*Name, OutPath, LogCategory);
	}

	FString WritePath;
	// Relative paths go to the profiling directory
	if (FPathViews::IsRelativePath(Path))
	{
		WritePath = FPaths::Combine(FPaths::ProfilingDir(), InPath);
	}
#if PLATFORM_WINDOWS
	// On windows we treat paths starting with '/' as relative, except double slash which is a network path
	else if (FPathViews::IsSeparator(Path[0]) && !(Path.Len() > 1 && FPathViews::IsSeparator(Path[1])))
	{
		WritePath = FPaths::Combine(FPaths::ProfilingDir(), InPath);
	}
#endif
	else
	{
		WritePath = InPath;
	}

	// If a directory is specified, add the default trace file name
	if (FPathViews::GetCleanFilename(WritePath).IsEmpty())
	{
		WritePath = FPaths::Combine(WritePath, GetDefaultName());
	}

	// The user may not have provided a suitable extension
	if (FPathViews::GetExtension(WritePath) != TEXT("utrace"))
	{
		WritePath = FPaths::SetExtension(WritePath, TEXT(".utrace"));
	}

	// Append timestamp to filename if requested via -tracefiletimestamps
	if (GTraceFileTimestamps)
	{
		const FString BasePath = FPaths::GetBaseFilename(WritePath, false); // false = include path
		const FDateTime Now = FDateTime::Now();
		WritePath = FString::Printf(TEXT("%s_%s.utrace"), *BasePath, *Now.ToString(TEXT("%Y%m%d_%H%M%S")));
	}

	// Finally make sure the path is platform friendly
	IFileManager& FileManager = IFileManager::Get();
	FString NativePath = FileManager.ConvertToAbsolutePathForExternalAppForWrite(*WritePath);

	// Ensure we can write the trace file appropriately
	const FString WriteDir = FPaths::GetPath(NativePath);
	if (!FPaths::IsDrive(WriteDir))
	{
		if (!FileManager.MakeDirectory(*WriteDir, true))
		{
			UE_LOG_REF(LogCategory, Warning, TEXT("Failed to create directory '%s'"), *WriteDir);
			return false;
		}
	}

	if (!bTruncateFile && FileManager.FileExists(*NativePath))
	{
		UE_LOG_REF(LogCategory, Warning, TEXT("Trace file '%s' already exists"), *NativePath);
		return false;
	}

	OutPath = MoveTemp(NativePath);
	return true;
}

////////////////////////////////////////////////////////////////////////////////
void FTraceAuxiliaryImpl::FrameBasedUpdate()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FTraceAux_Trace_Update);

#if TRACE_PRIVATE_ALLOW_TCP_CONTROL || TRACE_PRIVATE_FULL_ENABLED
	UE::Trace::Update();
#else
	if (UE::Trace::IsTracing())
	{
		UE::Trace::Update();
		return;
	}
	
	static uint8 FrameCounter = 0;
	static constexpr uint8 DrainFrameInterval = 10;

	if (++FrameCounter >= DrainFrameInterval)
	{
		UE::Trace::Update();
		FrameCounter = 0;
	}
#endif
}

////////////////////////////////////////////////////////////////////////////////
void FTraceAuxiliaryImpl::UpdateStats() 
{
#if UE_TRACE_STATISTICS_ENABLED 
	UE::Trace::FStatistics Stats;
	UE::Trace::GetStatistics(Stats);
	EmitStats(Stats);
	EmitDiagnostics(Stats);
#endif
}

////////////////////////////////////////////////////////////////////////////////
void FTraceAuxiliaryImpl::EmitStats(const UE::Trace::FStatistics& Stats)
{
#if UE_TRACEAUX_EMIT_LLM_STATS && ENABLE_LOW_LEVEL_MEM_TRACKER
	if (FLowLevelMemTracker::Get().IsEnabled())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FTraceAux_Stats_LLM);
		FLowLevelMemTracker::Get().SetTagAmountForTracker(ELLMTracker::Default, LLM_TAG_NAME(TraceLog), ELLMTagSet::None, Stats.MemoryUsed, true);
		FLowLevelMemTracker::Get().SetTagAmountForTracker(ELLMTracker::Default, LLM_TAG_NAME(TraceLog_BlockPool), ELLMTagSet::None, Stats.BlockPoolAllocated, true);
		FLowLevelMemTracker::Get().SetTagAmountForTracker(ELLMTracker::Default, LLM_TAG_NAME(TraceLog_FixedBuffers), ELLMTagSet::None, Stats.FixedBufferAllocated, true);
		FLowLevelMemTracker::Get().SetTagAmountForTracker(ELLMTracker::Default, LLM_TAG_NAME(TraceLog_SharedBuffers), ELLMTagSet::None, Stats.SharedBufferAllocated, true);
		FLowLevelMemTracker::Get().SetTagAmountForTracker(ELLMTracker::Default, LLM_TAG_NAME(TraceLog_Cache), ELLMTagSet::None, Stats.CacheAllocated, true);
	}
#endif

#if UE_TRACEAUX_EMIT_CSV_STATS
	// Only publish CSV stats if we have ever run tracing in order to reduce overhead in most runs.
	static bool bDoCsvStats = false;
	if (IsConnected() || bDoCsvStats)
	{
		bDoCsvStats = true;

		TRACE_CPUPROFILER_EVENT_SCOPE(FTraceAux_Stats_CSV);
		CSV_CUSTOM_STAT(Trace, MemoryUsedMb,    double(Stats.MemoryUsed)            / 1024.0 / 1024.0, ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT(Trace, BlockPoolMb,     double(Stats.BlockPoolAllocated)    / 1024.0 / 1024.0, ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT(Trace, FixedBuffersMb,  double(Stats.FixedBufferAllocated)  / 1024.0 / 1024.0, ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT(Trace, SharedBuffersMb, double(Stats.SharedBufferAllocated) / 1024.0 / 1024.0, ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT(Trace, CacheMb,         double(Stats.CacheAllocated)        / 1024.0 / 1024.0, ECsvCustomStatOp::Set);
	}
#endif
}

////////////////////////////////////////////////////////////////////////////////
void FTraceAuxiliaryImpl::EmitDiagnostics(const UE::Trace::FStatistics& Stats)
{
	static double LastUpdate = FPlatformTime::Seconds();
	const double CurrentTime = FPlatformTime::Seconds();
	static uint64 LastSent = Stats.BytesSent;
	static uint64 LastTraced = Stats.BytesTraced;
	static uint64 LastEmitted = Stats.BytesEmitted;
	const double Delta = FMath::Clamp(CurrentTime - LastUpdate, UE_DOUBLE_SMALL_NUMBER, UE_DOUBLE_BIG_NUMBER);
	const double TracedPerSecond = double(Stats.BytesTraced - LastTraced) / Delta;
	const double SentPerSecond = double(Stats.BytesSent - LastSent) / Delta;
	const double EmittedPerSecond = double(Stats.BytesEmitted - LastEmitted) / Delta;

	LastSent = Stats.BytesSent;
	LastTraced = Stats.BytesTraced;
	LastEmitted = Stats.BytesEmitted;
	LastUpdate = CurrentTime;

	// Generate a diagnostics message (can be rendered to screen) if the block
	// pool is at the limit and utilization is above warning threshhold.
	static uint32 Sticky = 0;
	constexpr uint32 Stickyness = 50; // Number of frames the message will "stick" after
	constexpr float BlockPoolUtilizationWarning = 0.8f; // Warn above 80% utilization
	constexpr float BlockPoolGrowthWarning = 0.8f; // Warn above 80% growth
	const float BlockPoolFreeFraction = Stats.BlockPoolAllocatedBlocks ? float(Stats.BlockPoolFreeBlocks)/float(Stats.BlockPoolAllocatedBlocks) : 1.0f;
	const float BlockPoolUtilization = 1.0f - BlockPoolFreeFraction;
	const float BlockPoolGrowth = GInitializeDesc.BlockPoolMaxSize ? float(Stats.BlockPoolAllocated) / float(GInitializeDesc.BlockPoolMaxSize) : 0.0f;

	if (BlockPoolGrowth > BlockPoolGrowthWarning && BlockPoolUtilization > BlockPoolUtilizationWarning)
	{
		Sticky = Stickyness;
	}

	if (Sticky)
	{
		TStringBuilder<128> Diagnostic;
		Diagnostic.Appendf(TEXT("Trace: Block pool %.2f %% full (Size: %.2f MiB, Max: %.2f MiB), Sending: %.2f MiB/s Emitting: %.2f Mib/s"),
			100.0f * BlockPoolUtilization,
			float(Stats.BlockPoolAllocated) / (1024*1024),
			float(GInitializeDesc.BlockPoolMaxSize) / (1024*1024),
			SentPerSecond / (1024 * 1024),
			EmittedPerSecond / (1024 * 1024)
		);
		Sticky = FMath::Clamp(Sticky - 1, 0u, Stickyness);
		
		{
			FWriteScopeLock _(DiagnosticsLock);
			DiagnosticMessage = Diagnostic.ToString();
		}

		if (!OnGetOnScreenMessageHandle.IsValid())
		{
			OnGetOnScreenMessageHandle = FCoreDelegates::OnGetOnScreenMessages.AddRaw(this, &FTraceAuxiliaryImpl::OnGetOnScreenMessages);
		}
	}
	else
	{
		if (!DiagnosticMessage.IsEmpty())
		{
			FWriteScopeLock _(DiagnosticsLock);
			DiagnosticMessage.Empty();
		}
	}
}

////////////////////////////////////////////////////////////////////////////////
void FTraceAuxiliaryImpl::OnGetOnScreenMessages(FCoreDelegates::FSeverityMessageMap& OutMessages)
{
	FReadScopeLock _(DiagnosticsLock);
	if (DiagnosticMessage.IsEmpty())
	{
		return;
	}
	
	OutMessages.Add(FCoreDelegates::EOnScreenMessageSeverity::Warning, FText::FromString(DiagnosticMessage));
}

////////////////////////////////////////////////////////////////////////////////
bool FTraceAuxiliaryImpl::WriteToFile(const TCHAR* Path, const FTraceAuxiliary::FLogCategoryAlias& LogCategory, uint16 SendFlags)
{
	FString NativePath;
	if (!FinalizeFilePath(Path, NativePath, LogCategory))
	{
		return false;
	}

	if (!UE::Trace::WriteTo(*NativePath, SendFlags))
	{
		if (FPathViews::Equals(NativePath, FStringView(Path)))
		{
			UE_LOG_REF(LogCategory, Warning, TEXT("Unable to trace to file '%s'"), *NativePath);
		}
		else
		{
			UE_LOG_REF(LogCategory, Warning, TEXT("Unable to trace to file '%s' (transformed from '%s')"), *NativePath, Path ? Path : TEXT("null"));
		}
		return false;
	}

	{
		FWriteScopeLock _(CurrentTargetLock);
		CurrentTraceTarget.TraceType = FTraceAuxiliary::EConnectionType::File;
		CurrentTraceTarget.TraceDest = MoveTemp(NativePath);
	}
	return true;
}

////////////////////////////////////////////////////////////////////////////////
bool FTraceAuxiliaryImpl::WriteSnapshot(const TCHAR* InFilePath, const FTraceAuxiliary::FLogCategoryAlias& LogCategory, uint32 MaxTailSize)
{
	double StartTime = FPlatformTime::Seconds();

	FString NativePath;
	if (!FinalizeFilePath(InFilePath, NativePath, LogCategory))
	{
		return false;
	}

	UE_LOG_REF(LogCategory, Log, TEXT("Writing trace snapshot to '%s'..."), *NativePath);

	const bool bResult = UE::Trace::WriteSnapshotTo(*NativePath, MaxTailSize);

	if (bResult)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FTraceAux_OnSnapshotSaved);
		FTraceAuxiliary::OnSnapshotSaved.Broadcast(FTraceAuxiliary::EConnectionType::File, NativePath);
		UE_LOG_REF(LogCategory, Display, TEXT("Trace snapshot generated in %.3f seconds to \"%s\"."), FPlatformTime::Seconds() - StartTime, *NativePath);
	}
	else
	{
		UE_LOG_REF(LogCategory, Error, TEXT("Failed to trace snapshot to \"%s\"."), *NativePath);
	}

	return bResult;
}

////////////////////////////////////////////////////////////////////////////////
bool FTraceAuxiliaryImpl::SendSnapshot(const TCHAR* InHost, uint32 InPort, const FTraceAuxiliary::FLogCategoryAlias& LogCategory, uint32 MaxTailSize)
{
	double StartTime = FPlatformTime::Seconds();

	// If no host is set, assume localhost
	if (!InHost)
	{
		InHost = TEXT("localhost");
	}

	UE_LOG_REF(LogCategory, Log, TEXT("Sending trace snapshot to '%s'..."), InHost);

	const bool bResult = UE::Trace::SendSnapshotTo(InHost, InPort, MaxTailSize);

	if (bResult)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FTraceAux_OnSnapshotSaved);
		FTraceAuxiliary::OnSnapshotSaved.Broadcast(FTraceAuxiliary::EConnectionType::Network, InHost);
		UE_LOG_REF(LogCategory, Display, TEXT("Trace snapshot generated in %.3f seconds to \"%s\"."), FPlatformTime::Seconds() - StartTime, InHost);
	}
	else
	{
		UE_LOG_REF(LogCategory, Error, TEXT("Failed to trace snapshot to \"%s\"."), InHost);
	}

	return bResult;
}

////////////////////////////////////////////////////////////////////////////////
FString FTraceAuxiliaryImpl::GetDest() const
{
	FReadScopeLock _(CurrentTargetLock);
	return CurrentTraceTarget.TraceDest;
}

////////////////////////////////////////////////////////////////////////////////
bool FTraceAuxiliaryImpl::IsConnected() const
{
	return UE::Trace::IsTracing();
}

////////////////////////////////////////////////////////////////////////////////
bool FTraceAuxiliaryImpl::IsConnected(FGuid& OutSessionGuid, FGuid& OutTraceGuid)
{
	uint32 SessionGuid[4];
	uint32 TraceGuid[4];
	if (UE::Trace::IsTracingTo(SessionGuid, TraceGuid))
	{
		OutSessionGuid = *(FGuid*)SessionGuid;
		OutTraceGuid = *(FGuid*)TraceGuid;
		return true;
	}
	return false;
}

////////////////////////////////////////////////////////////////////////////////
FTraceAuxiliary::EConnectionType FTraceAuxiliaryImpl::GetConnectionType() const
{
	FReadScopeLock _(CurrentTargetLock);
	return CurrentTraceTarget.TraceType;
}

////////////////////////////////////////////////////////////////////////////////
void FTraceAuxiliaryImpl::GetActiveChannelsString(FStringBuilderBase& String) const
{
	UE::Trace::EnumerateChannels([](const ANSICHAR* Name, bool bEnabled, void* User)
	{
		FStringBuilderBase& EnabledChannelsStr = *static_cast<FStringBuilderBase*>(User);
		if (bEnabled)
		{
			FAnsiStringView NameView = FAnsiStringView(Name).LeftChop(7); // Remove "Channel" suffix
			EnabledChannelsStr << NameView << TEXT(",");
		}
	}, &String);
}

////////////////////////////////////////////////////////////////////////////////
void FTraceAuxiliaryImpl::StartWorkerThread()
{
	const bool bIsForkSafe = !FForkProcessHelper::IsForkedChildProcess() || FForkProcessHelper::IsForkedMultithreadInstance();
	const bool bShouldStart = GWorkerThreadConfig == EWorkerThreadConfig::OnDemand && bIsForkSafe;
	if (!bWorkerThreadStarted && bShouldStart)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FTraceAux_StartWorkerThread);
		// Remove end frame delegate as we don't need it anymore
		FCoreDelegates::OnEndFrame.Remove(EndFrameUpdateHandle);
		UE::Trace::StartWorkerThread();
		bWorkerThreadStarted = true;
	}
}

////////////////////////////////////////////////////////////////////////////////
void FTraceAuxiliaryImpl::StartEndFrameUpdate()
{
	EndFrameUpdateHandle = FCoreDelegates::OnEndFrame.AddStatic(&FTraceAuxiliaryImpl::FrameBasedUpdate);
}

////////////////////////////////////////////////////////////////////////////////
void TraceAuxiliaryOnTraceUpdateCallback()
{
	// This is called each time the Trace system updates.
	// If the trace system uses a worker thread, this is called from the worker thread.

#if ENABLE_LOW_LEVEL_MEM_TRACKER
	if (!FLowLevelMemTracker::Get().IsConfigured())
	{
		// a) Avoids emitting LLM stats before LLM is configured.
		// b) Avoids allocating memory (through the STAT or CounterTrace APIs) before LLM is configured.
		return;
	}
#endif

	GTraceAuxiliary.UpdateStats();

}

////////////////////////////////////////////////////////////////////////////////
void TraceAuxiliaryOnEndFrameCallback()
{
	// This is called at end of each frame.
	TRACE_CPUPROFILER_EVENT_SCOPE(FTraceAux_OnEndFrame_Stats);

	// Reset the trace update callback. From now on, we only trace stats once per frame.
	UE::Trace::SetUpdateCallback(nullptr);

	GTraceAuxiliary.UpdateStats();
}

////////////////////////////////////////////////////////////////////////////////
void FTraceAuxiliaryImpl::RegisterEndFrameCallbacks()
{
	// Update stats every frame.
	GEndFrameDelegateHandle = FCoreDelegates::OnEndFrame.AddStatic(TraceAuxiliaryOnEndFrameCallback);
}

////////////////////////////////////////////////////////////////////////////////
void TraceAuxiliaryOnConnectionCallback()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FTraceAux_OnConnection);
	GTraceAuxiliary.StartWorkerThread();
	FTraceAuxiliary::OnConnection.Broadcast();
}

////////////////////////////////////////////////////////////////////////////////
void TraceAuxiliaryOnCloseCallback() 
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FTraceAux_OnClosed);
	GTraceAuxiliary.OnClosed();
	FTraceAuxiliary::OnClosed.Broadcast();
}

////////////////////////////////////////////////////////////////////////////////
void TraceAuxiliaryOnMessageCallback(const UE::Trace::FMessageEvent& Message)
{
	#define LOG_WITH_VERBOSITY(Verbosity, Text) \
		UE_LOGF(LogTrace, Verbosity, "%ls", Text)

	TStringBuilder<128> LogStr;
	if (Message.Description)
	{
		LogStr << Message.Description;
	}
	const TCHAR* Msg = LogStr.ToString();

	switch (Message.Type)
	{
	case UE::Trace::EMessageType::VeryVerbose:
		LOG_WITH_VERBOSITY(VeryVerbose, Msg);
		break;
	case UE::Trace::EMessageType::Verbose:
		LOG_WITH_VERBOSITY(Verbose, Msg);
		break;
	case UE::Trace::EMessageType::Log:
		LOG_WITH_VERBOSITY(Log, Msg);
		break;
	case UE::Trace::EMessageType::Display:
		LOG_WITH_VERBOSITY(Display, Msg);
		break;
	case UE::Trace::EMessageType::PanicError:
		LOG_WITH_VERBOSITY(Warning, Msg);
		GTraceAuxiliary.SetHasPanicked();
		break;
	default:
		{
			if (Message.Type > UE::Trace::EMessageType::FatalStart)
			{
				LOG_WITH_VERBOSITY(Fatal, Msg);
			}
			else if (Message.Type > UE::Trace::EMessageType::ErrorStart)
			{
				LOG_WITH_VERBOSITY(Error, Msg);
			}
			else if (Message.Type > UE::Trace::EMessageType::WarningStart)
			{
				LOG_WITH_VERBOSITY(Warning, Msg);
			}
		}
		break;
	}

	#undef LOG_WITH_VERBOSITY
}

////////////////////////////////////////////////////////////////////////////////
void TraceAuxiliaryOnScopeBeginCallback(const ANSICHAR* ScopeName)
{
#if CPUPROFILERTRACE_ENABLED
	FCpuProfilerTrace::OutputBeginDynamicEvent(ScopeName);
#endif
}

////////////////////////////////////////////////////////////////////////////////
void TraceAuxiliaryOnScopeEndCallback()
{
#if CPUPROFILERTRACE_ENABLED
	FCpuProfilerTrace::OutputEndEvent();
#endif
}

////////////////////////////////////////////////////////////////////////////////
static void TraceAuxiliarySetupInitFromConfig(UE::Trace::FInitializeDesc& OutDesc)
{
	if (!GConfig)
	{
		return;
	}

	// Note that these options can only be used when tracing from a forked process (e.g. server).
	// For a regular process use command line argument -TraceThreadSleepTime and -TraceTailMb

	int32 SleepTimeConfig = 0;
	if (GConfig->GetInt(GTraceConfigSection, TEXT("SleepTimeInMS"), SleepTimeConfig, GEngineIni))
	{
		if (SleepTimeConfig > 0)
		{
			OutDesc.ThreadSleepTimeInMS = SleepTimeConfig;
		}
	}

	int32 TailSizeBytesConfig = 0;
	if (GConfig->GetInt(GTraceConfigSection, TEXT("TailSizeBytes"), TailSizeBytesConfig, GEngineIni))
	{
		OutDesc.TailSizeBytes = TailSizeBytesConfig;
	}
}

////////////////////////////////////////////////////////////////////////////////
static void TraceAuxiliaryConnectEpilogue()
{
	// Give the user some feedback that everything's underway.
	TStringBuilder<128> Channels;
	GTraceAuxiliary.GetActiveChannelsString(Channels);

	UE_LOGF(LogConsoleResponse, Display, "Enabled channels: %ls", Channels.ToString());
}

////////////////////////////////////////////////////////////////////////////////
static void TraceAuxiliarySend(const TArray<FString>& Args)
{
	if (Args.Num() < 1)
	{
		UE_LOGF(LogConsoleResponse, Warning, "No host name given. Usage: Trace.Send <Host> [ChannelSet]");
		return;
	}

	const TCHAR* Target = *Args[0];
	const TCHAR* Channels = Args.Num() > 1 ? *Args[1] : nullptr;
	if (FTraceAuxiliary::Start(FTraceAuxiliary::EConnectionType::Network, Target, Channels, nullptr, LogConsoleResponse))
	{
		TraceAuxiliaryConnectEpilogue();
	}
}

////////////////////////////////////////////////////////////////////////////////
static void TraceAuxiliaryFile(const TArray<FString>& Args)
{
	const TCHAR* Filepath = nullptr;
	const TCHAR* Channels = nullptr;

	if (Args.Num() == 2)
	{
		Filepath = *Args[0];
		Channels = *Args[1];
	}
	else if (Args.Num() == 1)
	{
		// Try to detect if the first argument is a file path.
		if (FCString::Strchr(*Args[0], TEXT('/')) ||
			FCString::Strchr(*Args[0], TEXT('\\')) ||
			FCString::Strchr(*Args[0], TEXT('.')) ||
			FCString::Strchr(*Args[0], TEXT(':')))
		{
			Filepath = *Args[0];
		}
		else
		{
			Channels = *Args[0];
		}
	}
	else if (Args.Num() > 2)
	{
		UE_LOGF(LogConsoleResponse, Warning, "Invalid arguments. Usage: Trace.File [Path] [ChannelSet]");
		return;
	}

	if (FTraceAuxiliary::Start(FTraceAuxiliary::EConnectionType::File, Filepath, Channels, nullptr, LogConsoleResponse))
	{
		TraceAuxiliaryConnectEpilogue();
	}
}

////////////////////////////////////////////////////////////////////////////////
static void TraceAuxiliaryStart(const TArray<FString>& Args)
{
	UE_LOGF(LogConsoleResponse, Warning, "'Trace.Start' is being deprecated in favor of 'Trace.File'.");
	TraceAuxiliaryFile(Args);
}

////////////////////////////////////////////////////////////////////////////////
static void TraceAuxiliaryStop()
{
	UE_LOGF(LogConsoleResponse, Display, "Tracing stopped.");
	GTraceAuxiliary.Stop(LogConsoleResponse);
}

////////////////////////////////////////////////////////////////////////////////
static void TraceAuxiliaryPause()
{
	UE_LOGF(LogConsoleResponse, Display, "Tracing paused.");
	GTraceAuxiliary.PauseChannels();
}

////////////////////////////////////////////////////////////////////////////////
static void TraceAuxiliaryResume()
{
	UE_LOGF(LogConsoleResponse, Display, "Tracing resumed.");
	GTraceAuxiliary.ResumeChannels();
}

////////////////////////////////////////////////////////////////////////////////
static void TraceAuxiliaryStatus()
{
	UE_LOGF(LogConsoleResponse, Display, "Trace status ----------------------------------------------------------");
	if (GTraceAuxiliary.GetHasPanicked())
	{
		UE_LOGF(LogConsoleResponse, Error, "Trace is in panic mode and no longer useable.");
	}
	else
	{
		// Status of data connection
		TStringBuilder<256> ConnectionStr;
		FGuid SessionGuid, TraceGuid;
		if (GTraceAuxiliary.IsConnected(SessionGuid, TraceGuid))
		{
			const FString Dest = GTraceAuxiliary.GetDest();
			if (!Dest.IsEmpty())
			{
				ConnectionStr.Appendf(TEXT("Tracing to '%s', "), *Dest);
				ConnectionStr.Appendf(TEXT("session %s trace %s"), *SessionGuid.ToString(), *TraceGuid.ToString());
			}
			else
			{
				// If GTraceAux doesn't know about the target but we are still tracing this is an externally initiated connection
				// (e.g. connection command from Insights).
				ConnectionStr = TEXT("Tracing to unknown target (externally set)");
			}
		}
		else
		{
			ConnectionStr = TEXT("Not tracing");
		}
		UE_LOGF(LogConsoleResponse, Display, "Connection: %ls", ConnectionStr.ToString());
	}

	// Stats
	UE::Trace::FStatistics Stats;
	UE::Trace::GetStatistics(Stats);
	constexpr double KiB = 1.0 / 1024.0;
	constexpr double MiB = KiB / 1024.0;
	UE_LOGF(LogConsoleResponse, Display, "Memory Used: %.04f MiB",
		double(Stats.MemoryUsed) * MiB);
	UE_LOGF(LogConsoleResponse, Display, "Block Pool: %.04f MiB",
		double(Stats.BlockPoolAllocated) * MiB);
	UE_LOGF(LogConsoleResponse, Display, "Fixed Buffers: %.04f MiB",
		double(Stats.FixedBufferAllocated) * MiB);
	UE_LOGF(LogConsoleResponse, Display, "Shared Buffers: %.04f MiB",
		double(Stats.SharedBufferAllocated) * MiB);
	UE_LOGF(LogConsoleResponse, Display, "Important Events Cache: %.04f MiB (%.02f MiB used + %0.02f MiB unused | %0.02f MiB waste)",
		double(Stats.CacheAllocated) * MiB,
		double(Stats.CacheUsed) * MiB,
		double(Stats.CacheAllocated - Stats.CacheUsed) * MiB,
		double(Stats.CacheWaste) * MiB);
	UE_LOGF(LogConsoleResponse, Display, "Thread Registry: %.04f KiB",
		double(Stats.ThreadRegistryAllocated) * KiB);
	UE_LOGF(LogConsoleResponse, Display, "Emitted: %.02f MiB",
		double(Stats.BytesEmitted) * MiB);
	UE_LOGF(LogConsoleResponse, Display, "Traced: %.02f MiB",
		double(Stats.BytesTraced) * MiB);
	int64 BytesEmittedNotTraced = int64(Stats.BytesEmitted) - int64(Stats.BytesTraced);
	UE_LOGF(LogConsoleResponse, Display, "Emitted - Traced: %.02f MiB (%lli bytes)",
		double(BytesEmittedNotTraced) * MiB,
		BytesEmittedNotTraced);
	UE_LOGF(LogConsoleResponse, Display, "Sent: %.02f MiB",
		double(Stats.BytesSent) * MiB);

	// Channels
	struct EnumerateType
	{
		TStringBuilder<512> ChannelsStr;
		uint32 Count = 0;
#if WITH_EDITOR
		uint32 LineLen = 50;
#else
		uint32 LineLen = 20;
#endif

		void AddChannel(const FAnsiStringView& NameView)
		{
			if (Count++ > 0)
			{
				ChannelsStr << TEXT(", ");
				LineLen += 2;
			}
			if (LineLen + NameView.Len() > 100)
			{
				ChannelsStr << TEXT("\n    ");
				LineLen = 4;
			}
			ChannelsStr << NameView;
			LineLen += NameView.Len();
		}
	} EnumerateUserData[2];
	UE::Trace::EnumerateChannels([](const ANSICHAR* Name, bool bEnabled, void* User)
	{
		EnumerateType* EnumerateUser = static_cast<EnumerateType*>(User);
		FAnsiStringView NameView = FAnsiStringView(Name).LeftChop(7); // Remove "Channel" suffix
		EnumerateUser[bEnabled ? 0 : 1].AddChannel(NameView);
	}, EnumerateUserData);
	UE_LOGF(LogConsoleResponse, Display, "Enabled channels: %ls", EnumerateUserData[0].Count == 0 ? TEXT("<none>") : EnumerateUserData[0].ChannelsStr.ToString());
	UE_LOGF(LogConsoleResponse, Display, "Available channels: %ls", EnumerateUserData[1].ChannelsStr.ToString());

	UE_LOGF(LogConsoleResponse, Display, "-----------------------------------------------------------------------");
}

////////////////////////////////////////////////////////////////////////////////
static void TraceAuxiliaryEnableChannels(const TArray<FString>& Args)
{
	if (Args.Num() == 0)
	{
		UE_LOGF(LogConsoleResponse, Warning, "Need to provide at least one channel.");
		return;
	}
	GTraceAuxiliary.EnableChannels(*Args[0], LogConsoleResponse);

	TStringBuilder<128> EnabledChannels;
	GTraceAuxiliary.GetActiveChannelsString(EnabledChannels);
	UE_LOGF(LogConsoleResponse, Display, "Enabled channels: %ls", EnabledChannels.ToString());
}

////////////////////////////////////////////////////////////////////////////////
static void TraceAuxiliaryDisableChannels(const TArray<FString>& Args)
{
	if (Args.Num() == 0)
	{
		GTraceAuxiliary.DisableChannels(nullptr, LogConsoleResponse);
	}
	else
	{
		GTraceAuxiliary.DisableChannels(*Args[0], LogConsoleResponse);
	}

	TStringBuilder<128> EnabledChannels;
	GTraceAuxiliary.GetActiveChannelsString(EnabledChannels);
	UE_LOGF(LogConsoleResponse, Display, "Enabled channels: %ls", EnabledChannels.ToString());
}

////////////////////////////////////////////////////////////////////////////////
static void TraceAuxiliarySnapshotFile(const TArray<FString>& Args)
{
	const TCHAR* FilePath = nullptr;

	if (Args.Num() == 1)
	{
		FilePath = *Args[0];
	}
	else if (Args.Num() > 1)
	{
		UE_LOGF(LogConsoleResponse, Warning, "Invalid arguments. Usage: Trace.SnapshotFile [Path]");
		return;
	}

	GTraceAuxiliary.WriteSnapshot(FilePath, LogConsoleResponse, 0);
}

////////////////////////////////////////////////////////////////////////////////
static void TraceAuxiliarySnapshotSend(const TArray<FString>& Args)
{
	const TCHAR* Host = nullptr;
	uint32 Port = 0;

	if (Args.Num() >= 1)
	{
		Host = *Args[0];
	}
	if (Args.Num() >= 2)
	{
		LexFromString(Port, *Args[1]);
	}
	if (Args.Num() > 2)
	{
		UE_LOGF(LogConsoleResponse, Warning, "Invalid arguments. Usage: Trace.SnapshotSend <Host> <Port>");
		return;
	}

	GTraceAuxiliary.SendSnapshot(Host, Port, LogConsoleResponse, 0);
}

////////////////////////////////////////////////////////////////////////////////
static void TraceBookmark(const TArray<FString>& Args)
{
	TRACE_BOOKMARK(TEXT("%s"), Args.Num() ? *Args[0] : TEXT(""));
}

////////////////////////////////////////////////////////////////////////////////
static void TraceRegionBegin(const TArray<FString>& Args)
{
	if (Args.Num() > 0)
	{
		TRACE_BEGIN_REGION(*FString::Join(Args, TEXT(" ")), TEXT("ConsoleCommandRegion"));
	}
}

////////////////////////////////////////////////////////////////////////////////
static void TraceRegionEnd(const TArray<FString>& Args)
{
	if (Args.Num() > 0)
	{
		TRACE_END_REGION(*FString::Join(Args, TEXT(" ")));
	}
}

////////////////////////////////////////////////////////////////////////////////
/// Console commands
////////////////////////////////////////////////////////////////////////////////
#if UE_TRACEAUX_CON_CMDS

////////////////////////////////////////////////////////////////////////////////
static FAutoConsoleCommand TraceAuxiliarySendCmd(
	TEXT("Trace.Send"),
	TEXT("<Host> [ChannelSet] - Starts tracing to a trace store."
		" <Host> is the IP address or hostname of the trace store."
		" ChannelSet is comma-separated list of trace channels/presets to be enabled."
	),
	FConsoleCommandWithArgsDelegate::CreateStatic(TraceAuxiliarySend)
);

////////////////////////////////////////////////////////////////////////////////
static FAutoConsoleCommand TraceAuxiliaryStartCmd(
	TEXT("Trace.Start"),
	TEXT("[ChannelSet] - (Deprecated: Use Trace.File instead.) Starts tracing to a file."
		" ChannelSet is comma-separated list of trace channels/presets to be enabled."
	),
	FConsoleCommandWithArgsDelegate::CreateStatic(TraceAuxiliaryStart)
);

////////////////////////////////////////////////////////////////////////////////
static FAutoConsoleCommand TraceAuxiliaryFileCmd(
	TEXT("Trace.File"),
	TEXT("[Path] [ChannelSet] - Starts tracing to a file."
		" ChannelSet is comma-separated list of trace channels/presets to be enabled."
		" Either Path or ChannelSet can be excluded."
	),
	FConsoleCommandWithArgsDelegate::CreateStatic(TraceAuxiliaryFile)
);

////////////////////////////////////////////////////////////////////////////////
static FAutoConsoleCommand TraceAuxiliaryStopCmd(
	TEXT("Trace.Stop"),
	TEXT("Stops tracing profiling events."),
	FConsoleCommandDelegate::CreateStatic(TraceAuxiliaryStop)
);

////////////////////////////////////////////////////////////////////////////////
static FAutoConsoleCommand TraceAuxiliaryPauseCmd(
	TEXT("Trace.Pause"),
	TEXT("Pauses all trace channels currently sending events."),
	FConsoleCommandDelegate::CreateStatic(TraceAuxiliaryPause)
);

////////////////////////////////////////////////////////////////////////////////
static FAutoConsoleCommand TraceAuxiliaryResumeCmd(
	TEXT("Trace.Resume"),
	TEXT("Resumes tracing that was previously paused (re-enables the paused channels)."),
	FConsoleCommandDelegate::CreateStatic(TraceAuxiliaryResume)
);

////////////////////////////////////////////////////////////////////////////////
static FAutoConsoleCommand TraceAuxiliaryStatusCmd(
	TEXT("Trace.Status"),
	TEXT("Prints Trace status to console."),
	FConsoleCommandDelegate::CreateStatic(TraceAuxiliaryStatus)
);

////////////////////////////////////////////////////////////////////////////////
static FAutoConsoleCommand TraceAuxiliaryChannelEnableCmd(
	TEXT("Trace.Enable"),
	TEXT("[ChannelSet] - Enables a set of channels."
		" ChannelSet is comma-separated list of trace channels/presets to be enabled."
	),
	FConsoleCommandWithArgsDelegate::CreateStatic(TraceAuxiliaryEnableChannels)
);

////////////////////////////////////////////////////////////////////////////////
static FAutoConsoleCommand TraceAuxiliaryChannelDisableCmd(
	TEXT("Trace.Disable"),
	TEXT("[ChannelSet] - Disables a set of channels."
		" ChannelSet is comma-separated list of trace channels/presets to be disabled."
		" If no channel is specified, it disables all channels."
	),
	FConsoleCommandWithArgsDelegate::CreateStatic(TraceAuxiliaryDisableChannels)
);

////////////////////////////////////////////////////////////////////////////////
static FAutoConsoleCommand TraceAuxiliarySnapshotFileCmd(
	TEXT("Trace.SnapshotFile"),
	TEXT("[Path] - Writes a snapshot of the current in-memory trace buffer to a file."
	),
	FConsoleCommandWithArgsDelegate::CreateStatic(TraceAuxiliarySnapshotFile)
);

////////////////////////////////////////////////////////////////////////////////
static FAutoConsoleCommand TraceAuxiliarySnapshotSendCmd(
	TEXT("Trace.SnapshotSend"),
	TEXT("<Host> <Port> - Sends a snapshot of the current in-memory trace buffer to a server. If no host is specified 'localhost' is used."
	),
	FConsoleCommandWithArgsDelegate::CreateStatic(TraceAuxiliarySnapshotSend)
);

////////////////////////////////////////////////////////////////////////////////
static FAutoConsoleCommand TraceBookmarkCmd(
	TEXT("Trace.Bookmark"),
	TEXT("[Name] - Emits a TRACE_BOOKMARK() event with the given string name."
	),
	FConsoleCommandWithArgsDelegate::CreateStatic(TraceBookmark)
);

////////////////////////////////////////////////////////////////////////////////
static FAutoConsoleCommand TraceRegionBeginCmd(
	TEXT("Trace.RegionBegin"),
	TEXT("[Name] - Emits a TRACE_BEGIN_REGION() event with the given string name."
	),
	FConsoleCommandWithArgsDelegate::CreateStatic(TraceRegionBegin)
);

////////////////////////////////////////////////////////////////////////////////
static FAutoConsoleCommand TraceRegionEndCmd(
	TEXT("Trace.RegionEnd"),
	TEXT("[Name] - Emits a TRACE_END_REGION() event with the given string name."
	),
	FConsoleCommandWithArgsDelegate::CreateStatic(TraceRegionEnd)
);

#endif // UE_TRACEAUX_CON_CMDS

#endif // UE_TRACE_ENABLED

////////////////////////////////////////////////////////////////////////////////
static bool StartFromCommandlineArguments(const TCHAR* CommandLine)
{
#if UE_TRACE_ENABLED

	// Get active channels
	FString Channels;
	if (FParse::Value(CommandLine, TEXT("-trace="), Channels, false))
	{
	}
	else if (FParse::Param(CommandLine, TEXT("trace")))
	{
		Channels = GDefaultChannels.ChannelList;
	}
#if WITH_EDITOR
	else
	{
		Channels = GDefaultChannels.ChannelList;
	}
#endif

	// By default, if any channels are enabled we trace to memory.
	FTraceAuxiliary::EConnectionType Type = FTraceAuxiliary::EConnectionType::None;

	// Setup options
	FTraceAuxiliary::FOptions Opts;
	Opts.bTruncateFile = FParse::Param(CommandLine, TEXT("tracefiletrunc"));
	GTraceFileTimestamps = FParse::Param(CommandLine, TEXT("tracefiletimestamps"));

	// Find if a connection type is specified
	FString Parameter;
	const TCHAR* Target = nullptr;
	if (FParse::Value(CommandLine, TEXT("-tracehost="), Parameter))
	{
		Type = FTraceAuxiliary::EConnectionType::Network;
		Target = *Parameter;
	}
	else if (FParse::Value(CommandLine, TEXT("-tracehost"), Parameter))
	{
		Type = FTraceAuxiliary::EConnectionType::Network;
		Target = TEXT("localhost");
	}
	else if (FParse::Value(CommandLine, TEXT("-tracefile="), Parameter))
	{
		Type = FTraceAuxiliary::EConnectionType::File;
		if (Parameter.IsEmpty())
		{
			UE_LOGF(LogTrace, Warning, "Empty parameter to 'tracefile' argument. Using default filename.");
			Target = nullptr;
		}
		else
		{
			Target = *Parameter;
		}
	}
	else if (FParse::Param(CommandLine, TEXT("tracefile")))
	{
		Type = FTraceAuxiliary::EConnectionType::File;
		Target = nullptr;
	}

	// If user has defined a connection type but not specified channels, use the default channel set.
	if (Type != FTraceAuxiliary::EConnectionType::None && Channels.IsEmpty())
	{
		Channels = GDefaultChannels.ChannelList;
	}

	if (Channels.IsEmpty())
	{
		return false;
	}

	if (!GTraceAutoStart)
	{
		GTraceAuxiliary.AddCommandlineChannels(*Channels);
		return false;
	}

	// Finally start tracing to the requested connection
	return FTraceAuxiliary::Start(Type, Target, *Channels, &Opts);

#else // UE_TRACE_ENABLED

	return false;

#endif // UE_TRACE_ENABLED
}

////////////////////////////////////////////////////////////////////////////////
FTraceAuxiliary::FOnConnection FTraceAuxiliary::OnConnection;
FTraceAuxiliary::FOnClosed FTraceAuxiliary::OnClosed;
FTraceAuxiliary::FOnTraceStarted FTraceAuxiliary::OnTraceStarted;
FTraceAuxiliary::FOnTraceStopped FTraceAuxiliary::OnTraceStopped;
FTraceAuxiliary::FOnSnapshotSaved FTraceAuxiliary::OnSnapshotSaved;

////////////////////////////////////////////////////////////////////////////////
static bool TraceAuxiliaryStartShared(const TCHAR* Channels, const FTraceAuxiliary::FOptions* Options, const FTraceAuxiliary::FLogCategoryAlias& LogCategory)
{
#if UE_TRACE_MINIMAL_ENABLED
	if (GTraceAuxiliary.IsParentProcessAndPreFork())
	{
		return false;
	}

	if (GTraceAuxiliary.IsConnected())
	{
		UE_LOG_REF(LogCategory, Error, TEXT("Unable to start trace, already tracing to %s"), *GTraceAuxiliary.GetDest());
		return false;
	}

	if (Channels)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FTraceAux_EnableChannels);
		UE_LOG_REF(LogCategory, Display, TEXT("Requested channels: '%s'"), Channels);
		GTraceAuxiliary.ResetCommandlineChannels();
		GTraceAuxiliary.AddCommandlineChannels(Channels);
		GTraceAuxiliary.EnableCommandlineChannels();
	}

	return true;
#else // UE_TRACE_MINIMAL_ENABLED
	return false;
#endif
}

////////////////////////////////////////////////////////////////////////////////
bool FTraceAuxiliary::Start(EConnectionType Type, const TCHAR* Target, const TCHAR* Channels, FOptions* Options, const FLogCategoryAlias& LogCategory)
{
#if UE_TRACE_MINIMAL_ENABLED
	// Use the Relay function to start a trace relay.
	check(Type != EConnectionType::Relay);

	TRACE_CPUPROFILER_EVENT_SCOPE(FTraceAux_Start);

	if (!TraceAuxiliaryStartShared(Channels, Options, LogCategory))
	{
		return false;
	}

	if (Options)
	{
		// Truncation is only valid when tracing to file and filename is set.
		if (Options->bTruncateFile && Type == EConnectionType::File && Target != nullptr)
		{
			GTraceAuxiliary.SetTruncateFile(Options->bTruncateFile);
		}
	}

	uint16 SendFlags = (Options && Options->bExcludeTail) ? UE::Trace::FSendFlags::ExcludeTail : 0;

	return GTraceAuxiliary.Connect(Type, Target, LogCategory, SendFlags);
#else // UE_TRACE_MINIMAL_ENABLED
	return false;
#endif
}

////////////////////////////////////////////////////////////////////////////////
bool FTraceAuxiliary::Relay(UPTRINT Handle, UE::Trace::IoWriteFunc WriteFunc, UE::Trace::IoCloseFunc CloseFunc, const TCHAR* Channels, const FOptions* Options)
{
#if UE_TRACE_MINIMAL_ENABLED
	TRACE_CPUPROFILER_EVENT_SCOPE(FTraceAux_Relay);

	if (!TraceAuxiliaryStartShared(Channels, Options, LogTrace))
	{
		return false;
	}

	uint16 SendFlags = (Options && Options->bExcludeTail) ? UE::Trace::FSendFlags::ExcludeTail : 0;

	return GTraceAuxiliary.ConnectRelay(Handle, WriteFunc, CloseFunc, SendFlags);
#else // UE_TRACE_MINIMAL_ENABLED
	return false;
#endif
}

#if UE_TRACE_ALLOW_SECURE_TRACING

////////////////////////////////////////////////////////////////////////////////
bool FTraceAuxiliary::StartSecure(FSecureOptions& SecureOptions, FOptions* Options, const TCHAR* Channels)
{
#if UE_TRACE_MINIMAL_ENABLED
	TRACE_CPUPROFILER_EVENT_SCOPE(FTraceAux_Secure);

	if (!TraceAuxiliaryStartShared(Channels, Options, LogTrace))
	{
		return false;
	}

	uint16 SendFlags = (Options && Options->bExcludeTail) ? UE::Trace::FSendFlags::ExcludeTail : 0;

	return GTraceAuxiliary.ConnectSecure(SecureOptions, SendFlags);
#else // UE_TRACE_MINIMAL_ENABLED
	return false;
#endif
}

////////////////////////////////////////////////////////////////////////////////
TArrayView<uint8> FTraceAuxiliary::GetSecureCertificate()
{
	uint8* Ptr;
	uint32 Size;
	if (!UE::Trace::SecureGetSelfSignedCertificate(&Ptr, &Size))
	{
		return TArrayView<uint8>();
	}
	return TArrayView<uint8>(Ptr, Size);
}

////////////////////////////////////////////////////////////////////////////////
FString FTraceAuxiliary::GetSecureHost()
{
	return GTraceAuxiliary.GetSecureHost();
}

////////////////////////////////////////////////////////////////////////////////
uint16 FTraceAuxiliary::GetSecurePort()
{
	return GTraceAuxiliary.GetSecurePort();
}

#endif // UE_TRACE_ALLOW_SECURE_TRACING

////////////////////////////////////////////////////////////////////////////////
bool FTraceAuxiliary::Stop()
{
#if UE_TRACE_MINIMAL_ENABLED
	return GTraceAuxiliary.Stop(LogTrace);
#else
	return false;
#endif
}

////////////////////////////////////////////////////////////////////////////////
bool FTraceAuxiliary::Pause()
{
#if UE_TRACE_MINIMAL_ENABLED
	GTraceAuxiliary.PauseChannels();
#endif
	return true;
}

////////////////////////////////////////////////////////////////////////////////
bool FTraceAuxiliary::IsPaused()
{
#if UE_TRACE_MINIMAL_ENABLED
	return GTraceAuxiliary.IsPaused();
#else
	return false;
#endif
}

////////////////////////////////////////////////////////////////////////////////
bool FTraceAuxiliary::Resume()
{
#if UE_TRACE_MINIMAL_ENABLED
	GTraceAuxiliary.ResumeChannels();
#endif
	return true;
}

////////////////////////////////////////////////////////////////////////////////
bool FTraceAuxiliary::WriteSnapshot(const TCHAR* InFilePath, uint32 MaxTailSize)
{
#if UE_TRACE_MINIMAL_ENABLED
	TRACE_CPUPROFILER_EVENT_SCOPE(FTraceAux_WriteSnapshot);
	return GTraceAuxiliary.WriteSnapshot(InFilePath, LogTrace, MaxTailSize);
#else
	return true;
#endif
}

////////////////////////////////////////////////////////////////////////////////
bool FTraceAuxiliary::SendSnapshot(const TCHAR* Host, uint32 Port, uint32 MaxTailSize)
{
#if UE_TRACE_MINIMAL_ENABLED
	TRACE_CPUPROFILER_EVENT_SCOPE(FTraceAux_SendSnapshot);
	return GTraceAuxiliary.SendSnapshot(Host, Port, LogTrace, MaxTailSize);
#else
	return true;
#endif
}

////////////////////////////////////////////////////////////////////////////////
#if UE_TRACE_MINIMAL_ENABLED
static void TraceAuxiliaryAddPostForkCallback(const TCHAR* CommandLine)
{
	if (GTraceAutoStart)
	{
		UE_LOGF(LogTrace, Display, "Trace not started in parent because forking is expected. Use -NoFakeForking to trace parent.");
	}

	checkf(!GOnPostForkHandle.IsValid(), TEXT("TraceAuxiliaryAddPostForkCallback should only be called once."));

	GOnPostForkHandle = FCoreDelegates::OnPostFork.AddLambda([](EForkProcessRole Role)
	{
		if (Role == EForkProcessRole::Child)
		{
			FString CmdLine = FCommandLine::Get();

			FTraceAuxiliary::Initialize(*CmdLine);
			FTraceAuxiliary::TryAutoConnect();

			// InitializePresets is needed in the regular startup phase since dynamically loaded modules can define
			// presets and channels and we need to enable those after modules have been loaded. In the case of forked
			// process all modules should already have been loaded.
			// FTraceAuxiliary::InitializePresets(*CmdLine);
		}
	});
}
#endif // UE_TRACE_MINIMAL_ENABLED

////////////////////////////////////////////////////////////////////////////////
#if UE_TRACE_MINIMAL_ENABLED
UE_TRACE_EVENT_BEGIN(Diagnostics, Session2, NoSync|Important)
	UE_TRACE_EVENT_FIELD(uint8, ConfigurationType)
	UE_TRACE_EVENT_FIELD(uint8, TargetType)
	UE_TRACE_EVENT_FIELD(uint32, Changelist)
	UE_TRACE_EVENT_FIELD(uint32[], InstanceId) // added in UE 5.5
	UE_TRACE_EVENT_FIELD(UE::Trace::AnsiString, Platform)
	UE_TRACE_EVENT_FIELD(UE::Trace::AnsiString, AppName)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, ProjectName)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, Branch)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, BuildVersion)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, EngineVersion) // added in UE 5.8
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, CommandLine)
	UE_TRACE_EVENT_FIELD(UE::Trace::AnsiString, VFSPaths) // added in UE 5.7
UE_TRACE_EVENT_END()
#endif // UE_TRACE_MINIMAL_ENABLED

////////////////////////////////////////////////////////////////////////////////
#if UE_TRACE_MINIMAL_ENABLED
static void OutputDiagnostics(const TCHAR* CommandLine)
{
	constexpr int InstanceIdSize = 4;
	FGuid InstanceGuid = FApp::GetInstanceId();
	uint32 InstanceId[InstanceIdSize];
	for (int Index = 0; Index < InstanceIdSize; ++Index)
	{
		InstanceId[Index] = InstanceGuid[Index];
	}

	const ANSICHAR* Platform = UE_STRINGIZE(UBT_COMPILED_PLATFORM);
	constexpr uint32 PlatformLen = UE_ARRAY_COUNT(UE_STRINGIZE(UBT_COMPILED_PLATFORM)) - 1;

	const ANSICHAR* AppName = UE_APP_NAME;
	constexpr uint32 AppNameLen = UE_ARRAY_COUNT(UE_APP_NAME) - 1;

	const TCHAR* ProjectName = TEXT("");
#if IS_MONOLITHIC && !IS_PROGRAM
	if (FApp::HasProjectName())
	{
		ProjectName = FApp::GetProjectName();
	}
#endif
	const uint32 ProjectNameLen = FCString::Strlen(ProjectName);

	const TCHAR* BuildVersion = BuildSettings::GetBuildVersion();
	const uint32 BuildVersionLen = FCString::Strlen(BuildVersion);

	const TCHAR* BranchName = BuildSettings::GetBranchName();
	const uint32 BranchNameLen = FCString::Strlen(BranchName);

	const TCHAR* EngineVersion = BuildSettings::GetEngineVersionString();
	const uint32 EngineVersionLen = FCString::Strlen(EngineVersion);

#if UE_TRACEAUX_FULL
	const uint32 CommandLineLen = FCString::Strlen(CommandLine);

	const char* VFSPaths = BuildSettings::GetVfsPaths();
	const uint32 VFSPathsLen = FCStringAnsi::Strlen(VFSPaths);
#endif

	const uint32 DataSize = 0
		+ (InstanceIdSize * sizeof(uint32))
		+ (PlatformLen * sizeof(ANSICHAR))
		+ (AppNameLen * sizeof(ANSICHAR))
		+ (ProjectNameLen * sizeof(TCHAR))
		+ (BuildVersionLen * sizeof(TCHAR))
		+ (BranchNameLen * sizeof(TCHAR))
		+ (EngineVersionLen * sizeof(TCHAR))
#if UE_TRACEAUX_FULL
		+ (CommandLineLen * sizeof(TCHAR))
		+ (VFSPathsLen * sizeof(ANSICHAR))
#endif
		;

	UE_TRACE_LOG(Diagnostics, Session2, UE::Trace::TraceLogChannel, DataSize)
		<< Session2.ConfigurationType(uint8(FApp::GetBuildConfiguration()))
		<< Session2.TargetType(uint8(FApp::GetBuildTargetType()))
		<< Session2.Changelist(BuildSettings::GetCurrentChangelist())
		<< Session2.InstanceId(InstanceId, InstanceIdSize)
		<< Session2.Platform(Platform, PlatformLen)
		<< Session2.AppName(AppName, AppNameLen)
		<< Session2.ProjectName(ProjectName, ProjectNameLen)
		<< Session2.BuildVersion(BuildVersion, BuildVersionLen)
		<< Session2.Branch(BranchName, BranchNameLen)
		<< Session2.EngineVersion(EngineVersion, EngineVersionLen)
#if UE_TRACEAUX_FULL
		<< Session2.CommandLine(CommandLine, CommandLineLen)
		<< Session2.VFSPaths(VFSPaths, VFSPathsLen)
#endif
		;
}
#endif // UE_TRACE_MINIMAL_ENABLED

////////////////////////////////////////////////////////////////////////////////
void FTraceAuxiliary::Initialize(const TCHAR* CommandLine)
{
	static bool bInitialized = false;
	checkf(!bInitialized, TEXT("FTraceAuxiliary may only be initialized once."));
	if (bInitialized)
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FTraceAux_Init);
	UE_MEMSCOPE(TRACE_TAG);

#if UE_TRACE_SERVER_LAUNCH_ENABLED && UE_TRACE_SERVER_CONTROLS_ENABLED
	// Auto launch Unreal Trace Server for certain configurations
	if (!(FParse::Param(CommandLine, TEXT("notraceserver")) || FParse::Param(CommandLine, TEXT("buildmachine"))))
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FTraceAux_Launch_UTS);
		FTraceServerControls::Start();
	}
#endif

#if UE_TRACE_MINIMAL_ENABLED

	// Setup message callback so we get feedback from TraceLog
	UE::Trace::SetMessageCallback(&TraceAuxiliaryOnMessageCallback);

#if UE_TRACEAUX_AUTOSTART
	FParse::Bool(CommandLine, TEXT("-traceautostart="), GTraceAutoStart);
	UE_LOGF(LogTrace, Verbose, "Trace auto start = %d.", GTraceAutoStart);
#endif

	if (GTraceAuxiliary.IsParentProcessAndPreFork())
	{
		UE_LOGF(LogTrace, Log, "Trace initialization skipped for parent process (pre fork).");

		GTraceAuxiliary.DisableChannels(nullptr, LogTrace);

		// Set our post fork callback up and return - children will pass through and Initialize when they're created.
		TraceAuxiliaryAddPostForkCallback(CommandLine);
		return;
	}

	// Only set this post fork if used.
	bInitialized = true;

	UE_LOGF(LogTrace, Log, "Initializing trace...");

	// Trace out information about this session. This is done before initialization,
	// so that it is always sent (all channels are enabled prior to initialization).
	OutputDiagnostics(CommandLine);

	// Attempt to send trace data somewhere from the command line. It perhaps
	// seems odd to do this before initializing Trace, but it is done this way
	// to support disabling the "important" cache without losing any events.
	const bool bWasConnectedFromCmdLine = StartFromCommandlineArguments(CommandLine);

	// Emit empty stats once (to ensure all stats/counters start from zero).
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FTraceAux_InitialStats);
		UE::Trace::FStatistics Stats;
		GTraceAuxiliary.EmitStats(Stats);
	}

	// Initialize Trace. The settings are stored in a static for posterity.
	UE::Trace::FInitializeDesc& Desc = GInitializeDesc;
	if (FParse::Param(CommandLine, TEXT("tracescopes")))
	{
		Desc.OnScopeBeginFunc = &TraceAuxiliaryOnScopeBeginCallback;
		Desc.OnScopeEndFunc = &TraceAuxiliaryOnScopeEndCallback;
	}
#if WITH_EDITOR
	Desc.TailSizeBytes = 32 << 20;
#endif
	TraceAuxiliarySetupInitFromConfig(Desc);

	Desc.bUseImportantCache = (FParse::Param(CommandLine, TEXT("tracenocache")) == false);
	Desc.OnConnectionFunc = &TraceAuxiliaryOnConnectionCallback;
	Desc.OnCloseFunc = &TraceAuxiliaryOnCloseCallback;
	Desc.OnUpdateFunc = &TraceAuxiliaryOnTraceUpdateCallback;

	// Determine if the worker thread is allowed
	if (!FGenericPlatformProcess::SupportsMultithreading() || FParse::Param(FCommandLine::Get(), TEXT("notracethreading")))
	{
		GWorkerThreadConfig = EWorkerThreadConfig::Never;
		Desc.bUseWorkerThread = false;
	}
	else
	{
		// If allowed start worker thread if trace is started using command line argument,
		// otherwise start on demand. For the editor we always want to start a worker thread
		// since there is a lot of time until frames are reliably pumped.
		Desc.bUseWorkerThread = bWasConnectedFromCmdLine || UE_EDITOR || IS_PROGRAM;
		GWorkerThreadConfig = Desc.bUseWorkerThread ? EWorkerThreadConfig::OnInit : EWorkerThreadConfig::OnDemand;
	}

	if (!Desc.bUseWorkerThread)
	{
		GTraceAuxiliary.StartEndFrameUpdate();
	}

	FGuid SessionGuid;
	if (!FParse::Value(CommandLine, TEXT("-tracesessionguid="), SessionGuid))
	{
		SessionGuid = FApp::GetSessionId();
	}
	FMemory::Memcpy((FGuid&)Desc.SessionGuid, SessionGuid);

	if (FParse::Value(CommandLine, TEXT("-tracetailmb="), Desc.TailSizeBytes))
	{
		Desc.TailSizeBytes <<= 20;
	}

	if (!FParse::Value(CommandLine, TEXT("-tracethreadsleeptime="), Desc.ThreadSleepTimeInMS))
	{
		// Memory tracing is very chatty. To reduce load on trace we'll speed up the
		// worker thread so it can clear events faster.
		extern bool MemoryTrace_IsActive();
		if (MemoryTrace_IsActive())
		{
			int32 SleepTimeMs = 5;
			if (GConfig)
			{
				GConfig->GetInt(GTraceConfigSection, TEXT("SleepTimeWhenMemoryTracingInMS"), SleepTimeMs, GEngineIni);
			}

			if (Desc.ThreadSleepTimeInMS)
			{
				SleepTimeMs = FMath::Min<uint32>(Desc.ThreadSleepTimeInMS, SleepTimeMs);
			}

			Desc.ThreadSleepTimeInMS = SleepTimeMs;
		}
	}

	{
		TRACE_CPUPROFILER_EVENT_SCOPE_STR("UE::Trace::Initialize");

		UE::Trace::Initialize(Desc);
	}

	// Workaround for the fact that even if StartFromCommandlineArguments will enable channels
	// specified by the commandline, UE::Trace::Initialize will reset all channels.
	GTraceAuxiliary.EnableCommandlineChannelsPostInitialize();

	check(IsInGameThread());
	UE::Trace::ThreadRegister(TEXT("GameThread"), FPlatformTLS::GetCurrentThreadId(), -1);

	// Setup known on connection callbacks
	OnConnection.AddStatic(FStringTrace::OnConnection);

	// Register end frame callbacks
	GTraceAuxiliary.RegisterEndFrameCallbacks();

	// Initialize Callstack tracing.
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FTraceAux_CallstackTrace_Create);

		// If for any reason, memory tracing has been initialized without callstack tracing,
		// we need to use the same allocator; otherwise we use the regular malloc.
		FMalloc* TraceAllocator = MemoryTrace_GetAllocator();
		CallstackTrace_Create(TraceAllocator ? TraceAllocator : (FMalloc*)UE::Private::GMalloc);
	}
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FTraceAux_CallstackTrace_Init);

		CallstackTrace_Initialize();
	}

	// Initialize Platform Events tracing.
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FTraceAux_PlatformEventsTrace_Init);

		// By default use 125 microseconds (8 kHz) for the stack sampling interval.
		uint32 Microseconds = 125;
		FParse::Value(CommandLine, TEXT("-samplinginterval="), Microseconds);
		FPlatformEventsTrace::Init(Microseconds);
		FPlatformEventsTrace::PostInit();
	}

	if (GTraceAutoStart)
	{
		FModuleManager::Get().OnModulesChanged().AddLambda([](FName Name, EModuleChangeReason Reason)
		{
			if (Reason == EModuleChangeReason::ModuleLoaded)
			{
				GTraceAuxiliary.EnableCommandlineChannels();
			}
		});
	}

	GTraceAuxiliary.FreezeReadOnlyChannels();
	UE_LOGF(LogTrace, Log, "Finished trace initialization.");
#endif //UE_TRACE_MINIMAL_ENABLED
}

////////////////////////////////////////////////////////////////////////////////
void FTraceAuxiliary::InitializePresets(const TCHAR* CommandLine)
{
#if UE_TRACE_ENABLED
	if (GTraceAuxiliary.IsParentProcessAndPreFork() || !GTraceAutoStart)
	{
		return;
	}

	// Second pass over trace arguments, this time to allow config defined presets
	// to be applied.
	FString Parameter;
	if (FParse::Value(CommandLine, TEXT("-trace="), Parameter, false))
	{
		GTraceAuxiliary.AddCommandlineChannels(*Parameter);
		GTraceAuxiliary.EnableCommandlineChannels();
	}
#endif
}

////////////////////////////////////////////////////////////////////////////////
void FTraceAuxiliary::Shutdown()
{
#if UE_TRACE_ENABLED
	if (GTraceAuxiliary.IsParentProcessAndPreFork())
	{
		return;
	}

	// Make sure all platform event functionality has shut down as on some
	// platforms it impacts whole system, even if application has terminated.
	FPlatformEventsTrace::Stop();
#endif
}

////////////////////////////////////////////////////////////////////////////////
void FTraceAuxiliary::EnableCommandlineChannels()
{
#if UE_TRACEAUX_CMD_ARGS
	GTraceAuxiliary.EnableCommandlineChannels();
#endif
}

void FTraceAuxiliary::EnableChannels(TConstArrayView<uint32> ChannelIds, TMap<uint32, FString>* OutErrors)
{
#if UE_TRACE_MINIMAL_ENABLED
	GTraceAuxiliary.EnableChannels(ChannelIds, OutErrors);
#endif
}

void FTraceAuxiliary::DisableChannels(TConstArrayView<uint32> ChannelIds, TMap<uint32, FString>* OutErrors)
{
#if UE_TRACE_MINIMAL_ENABLED
	GTraceAuxiliary.DisableChannels(ChannelIds, OutErrors);
#endif
}

void FTraceAuxiliary::EnableChannels(const TCHAR* Channels)
{
#if UE_TRACE_MINIMAL_ENABLED
	GTraceAuxiliary.EnableChannels(Channels, LogTrace);
#endif
}

void FTraceAuxiliary::DisableChannels(const TCHAR* Channels)
{
#if UE_TRACE_MINIMAL_ENABLED
	GTraceAuxiliary.DisableChannels(Channels, LogTrace);
#endif
}

FString FTraceAuxiliary::GetTraceDestinationString()
{
#if UE_TRACE_MINIMAL_ENABLED
	return GTraceAuxiliary.GetDest();
#else
	return FString();
#endif
}

bool FTraceAuxiliary::IsConnected()
{
#if UE_TRACE_MINIMAL_ENABLED
	return GTraceAuxiliary.IsConnected();
#else
	return false;
#endif
}

bool FTraceAuxiliary::IsConnected(FGuid& OutSessionGuid, FGuid& OutTraceGuid)
{
#if UE_TRACE_MINIMAL_ENABLED
	return GTraceAuxiliary.IsConnected(OutSessionGuid, OutTraceGuid);
#else
	return false;
#endif
}

FTraceAuxiliary::EConnectionType FTraceAuxiliary::GetConnectionType()
{
#if UE_TRACE_MINIMAL_ENABLED
	return GTraceAuxiliary.GetConnectionType();
#else
	return FTraceAuxiliary::EConnectionType::None;
#endif
}

void FTraceAuxiliary::GetActiveChannelsString(FStringBuilderBase& String)
{
#if UE_TRACE_ENABLED
	GTraceAuxiliary.GetActiveChannelsString(String);
#endif
}

void FTraceAuxiliary::Panic()
{
	UE::Trace::Panic();
}

UE::Trace::FInitializeDesc const* FTraceAuxiliary::GetInitializeDesc()
{
#if UE_TRACE_ENABLED
	return &GInitializeDesc;
#else
	return nullptr;
#endif
}

void FTraceAuxiliary::EnumerateFixedChannelPresets(PresetCallback Callback)
{
#if UE_TRACE_ENABLED
	if (Callback(GDefaultChannels) == EEnumerateResult::Stop)
	{
		return;
	}

	if (Callback(GMemoryChannels) == EEnumerateResult::Stop)
	{
		return;
	}

	if (Callback(GMemoryLightChannels) == EEnumerateResult::Stop)
	{
		return;
	}
#endif
}

void FTraceAuxiliary::EnumerateChannelPresetsFromSettings(PresetCallback Callback)
{
#if UE_TRACE_ENABLED
	TArray<FString> PresetStrings;
	GConfig->GetSection(TEXT("Trace.ChannelPresets"), PresetStrings, GEngineIni);

	for (const FString& Item : PresetStrings)
	{
		FString Key, Value;
		Item.Split(TEXT("="), &Key, &Value);

		FChannelPreset Preset(*Key, *Value, false);
		if (Callback(Preset) == EEnumerateResult::Stop)
		{
			return;
		}
	}
#endif
}

FTraceAuxiliary::ETraceSystemStatus FTraceAuxiliary::GetTraceSystemStatus()
{
#if UE_TRACE_MINIMAL_ENABLED
	EConnectionType ConnectionType = GetConnectionType();
	if (ConnectionType == EConnectionType::Network)
	{
		return ETraceSystemStatus::TracingToServer;
	}
	else if (ConnectionType == EConnectionType::File)
	{
		return ETraceSystemStatus::TracingToFile;
	}
	else if (ConnectionType == EConnectionType::Relay)
	{
		return ETraceSystemStatus::TracingToCustomRelay;
	}
	else
	{
		return ETraceSystemStatus::Available;
	}
#else
	return  ETraceSystemStatus::NotAvailable;
#endif
}

////////////////////////////////////////////////////////////////////////////////
void FTraceAuxiliary::TryAutoConnect()
{
#if UE_TRACEAUX_AUTOSTART
	TRACE_CPUPROFILER_EVENT_SCOPE(FTraceAux_TryAutoConnect);
#if PLATFORM_WINDOWS
	if (GTraceAutoStart && !IsConnected() && !GTraceAuxiliary.IsParentProcessAndPreFork())
	{
		// If we can detect a named event it means UnrealInsights (Browser Mode) is running.
		// In this case, we try to auto-connect with the Trace Server.
		HANDLE KnownEvent = ::OpenEvent(EVENT_ALL_ACCESS, false, TEXT("Local\\UnrealInsightsAutoConnect"));
		if (KnownEvent != nullptr)
		{
			UE_LOGF(LogTrace, Display, "Unreal Insights instance detected, auto-connecting to local trace server...");
			GTraceAuxiliary.StartWorkerThread();
			Start(EConnectionType::Network, TEXT("127.0.0.1"), GTraceAuxiliary.HasCommandlineChannels() ? nullptr : TEXT("default"), nullptr);
			::CloseHandle(KnownEvent);
		}
	}
#elif PLATFORM_MAC || PLATFORM_LINUX
	if (GTraceAutoStart && !IsConnected() && !GTraceAuxiliary.IsParentProcessAndPreFork())
	{
		sem_t* AutoConnectSemaphore = sem_open("/UnrealInsightsAutoConnect", O_RDONLY);
		if (AutoConnectSemaphore != SEM_FAILED)
		{
			UE_LOGF(LogTrace, Display, "Unreal Insights instance detected, auto-connecting to local trace server...");
			Start(EConnectionType::Network, TEXT("127.0.0.1"), GTraceAuxiliary.HasCommandlineChannels() ? nullptr : TEXT("default"), nullptr);
			sem_close(AutoConnectSemaphore);
		}
	}
#endif // PLATFORMS
#endif // UE_TRACEAUX_AUTOSTART
}

////////////////////////////////////////////////////////////////////////////////
/// UnrealTraceServer launching
/////////////////////////////////////////////////////////////////////////////////
#if UE_TRACE_SERVER_CONTROLS_ENABLED

enum class ELaunchTraceServerCommand
{
	Fork,
	Kill
};

#if PLATFORM_WINDOWS
bool LaunchTraceServerCommand(ELaunchTraceServerCommand Command, bool bAddSponsor)
{
	FString FilePath = FPaths::Combine(FPaths::EngineDir(), TEXT("Binaries/Win64/UnrealTraceServer.exe"));
	if (!FPaths::FileExists(FilePath))
	{
		UE_LOGF(LogCore, Display, "UTS: The Unreal Trace Server binary is not available ('%ls')", *FilePath);
		return false;
	}

	TWideStringBuilder<MAX_PATH + 32> CreateProcArgs;
	CreateProcArgs << TEXT("\"") << FilePath << TEXT("\"");
	if (Command == ELaunchTraceServerCommand::Fork)
	{
		CreateProcArgs << TEXT(" fork");
	}
	else if (Command == ELaunchTraceServerCommand::Kill)
	{
		CreateProcArgs << TEXT(" kill");
	}
	if (bAddSponsor)
	{
		CreateProcArgs << TEXT(" --sponsor ") << FPlatformProcess::GetCurrentProcessId();
	}

	uint32 CreateProcFlags = 0;
	JOBOBJECT_BASIC_LIMIT_INFORMATION JobLimits;
	if (QueryInformationJobObject(NULL, JobObjectBasicLimitInformation, &JobLimits, sizeof(JOBOBJECT_BASIC_LIMIT_INFORMATION), NULL))
	{
		if (JobLimits.LimitFlags & JOB_OBJECT_LIMIT_BREAKAWAY_OK)
		{
			CreateProcFlags |= CREATE_BREAKAWAY_FROM_JOB;
		}
	}
	if (FParse::Param(FCommandLine::Get(), TEXT("traceshowstore")))
	{
		CreateProcFlags |= CREATE_NEW_CONSOLE;
	}
	else
	{
		CreateProcFlags |= CREATE_NO_WINDOW;
	}

	STARTUPINFOW StartupInfo = { sizeof(STARTUPINFOW) };
	PROCESS_INFORMATION ProcessInfo = {};

	const BOOL bOk = CreateProcessW(nullptr, LPWSTR(*CreateProcArgs), nullptr, nullptr,
									false, CreateProcFlags, nullptr, nullptr, &StartupInfo, &ProcessInfo);

	if (!bOk)
	{
		DWORD LastError = GetLastError();
		TCHAR ErrorBuffer[1024];
		FWindowsPlatformMisc::GetSystemErrorMessage(ErrorBuffer, UE_ARRAY_COUNT(ErrorBuffer), LastError);
		UE_LOGF(LogCore, Warning, "UTS: Unable to launch the Unreal Trace Server with '%ls'. %ls Error: 0x%X (%u)", *CreateProcArgs, ErrorBuffer, LastError, LastError);
		return false;
	}

	bool bSuccess = false;
	if (WaitForSingleObject(ProcessInfo.hProcess, 5000) == WAIT_TIMEOUT)
	{
		UE_LOGF(LogCore, Warning, "UTS: Timed out waiting for the Unreal Trace Server process to start");
	}
	else
	{
		DWORD ExitCode = 0x0000'a9e0;
		GetExitCodeProcess(ProcessInfo.hProcess, &ExitCode);
		if (ExitCode)
		{
			UE_LOGF(LogCore, Warning, "UTS: Unreal Trace Server process returned an error (0x%08x)", ExitCode);
		}
		else
		{
			if (Command == ELaunchTraceServerCommand::Kill)
			{
				UE_LOGF(LogCore, Log, "UTS: Unreal Trace Server was stopped");
			}
			else
			{
				UE_LOGF(LogCore, Log, "UTS: Unreal Trace Server launched successfully");
			}
			bSuccess = true;
		}
	}

	CloseHandle(ProcessInfo.hProcess);
	CloseHandle(ProcessInfo.hThread);

	return bSuccess;
}
#endif // PLATFORM_WINDOWS

#if PLATFORM_LINUX || PLATFORM_MAC
static bool LaunchTraceServerCommand(ELaunchTraceServerCommand Command, bool bAddSponsor)
{
#if USING_THREAD_SANITISER
	// TSAN doesn't like fork(), so disable this for now.
	return false;
#else // !USING_THREAD_SANITISER

	TAnsiStringBuilder<320> BinPath;
	BinPath << TCHAR_TO_UTF8(*FPaths::EngineDir());
#if PLATFORM_UNIX
	BinPath << "Binaries/Linux/UnrealTraceServer";
#elif PLATFORM_MAC
	BinPath << "Binaries/Mac/UnrealTraceServer";
#endif
	BinPath.ToString(); //Ensure zero termination

	if (access(*BinPath, F_OK) < 0)
	{
		UE_LOGF(LogCore, Display, "UTS: The Unreal Trace Server binary is not available ('%ls')", ANSI_TO_TCHAR(*BinPath));
		return false;
	}

	TAnsiStringBuilder<64> ForkArg;
	TAnsiStringBuilder<64> SponsorArg;
	if (Command == ELaunchTraceServerCommand::Fork)
	{
		ForkArg << "fork";
	}
	else if (Command == ELaunchTraceServerCommand::Kill)
	{
		ForkArg << "kill";
	}
	if (bAddSponsor)
	{
		SponsorArg  << "--sponsor=" << FPlatformProcess::GetCurrentProcessId();
	}
	ForkArg.ToString(); //Ensure zero termination
	SponsorArg.ToString();

	pid_t UtsPid = fork();
	if (UtsPid < 0)
	{
		UE_LOGF(LogCore, Warning, "UTS: Unable to fork (errno: %d)", errno);
		return false;
	}
	else if (UtsPid == 0)
	{
		// Launch UTS from the child process.
		char* Args[] = { BinPath.GetData(), ForkArg.GetData(), SponsorArg.GetData(), nullptr };
		extern char** environ;
		execve(*BinPath, Args, environ);
		_exit(0x80 | (errno & 0x7f));
	}

	// Wait until the child process finishes.
	int32 WaitStatus = 0;
	do
	{
		int32 WaitRet = waitpid(UtsPid, &WaitStatus, 0);
		if (WaitRet < 0)
		{
			UE_LOGF(LogCore, Warning, "UTS: waitpid() error (errno: %d)", errno);
			return false;
		}
	}
	while (!WIFEXITED(WaitStatus));

	int32 UtsRet = WEXITSTATUS(WaitStatus);
	if (UtsRet)
	{
		UE_LOGF(LogCore, Warning, "UTS: Unreal Trace Server process returned an error (0x%08x)", UtsRet);
		return false;
	}
	else
	{
		if (Command == ELaunchTraceServerCommand::Kill)
		{
			UE_LOGF(LogCore, Log, "UTS: Unreal Trace Server was stopped");
		}
		else
		{
			UE_LOGF(LogCore, Log, "UTS: Unreal Trace Server launched successfully");
		}
		return true;
	}
#endif // !USING_THREAD_SANITISER
}
#endif // PLATFORM_LINUX || PLATFORM_MAC

////////////////////////////////////////////////////////////////////////////////
bool FTraceServerControls::Start()
{
	return LaunchTraceServerCommand(ELaunchTraceServerCommand::Fork, true);
}

////////////////////////////////////////////////////////////////////////////////
bool FTraceServerControls::Stop()
{
	return LaunchTraceServerCommand(ELaunchTraceServerCommand::Kill, false);
}

#endif // UE_TRACE_SERVER_CONTROLS_ENABLED

////////////////////////////////////////////////////////////////////////////////
