// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProfilingDebugging/CallstackTrace.h"

#if UE_CALLSTACK_TRACE_ENABLED

#ifndef UE_CALLSTACK_TRACE_UNIX_USE_STACK_FRAMES_WALKING
	#define UE_CALLSTACK_TRACE_UNIX_USE_STACK_FRAMES_WALKING 0
#endif

#include "CoreTypes.h"
#include "HAL/Platform.h"
#include "HAL/PlatformStackWalk.h"
#include "ProfilingDebugging/CallstackTracePrivate.h"

////////////////////////////////////////////////////////////////////////////////
class FBacktracer
{
public:
	FBacktracer(FMalloc* InMalloc);
	~FBacktracer();
	static FBacktracer*	Get();
	uint32 GetBacktraceId(void* ReturnAddress);

private:
	static FBacktracer* Instance;
	FMalloc* Malloc;
	FCallstackTracer CallstackTracer;
};

////////////////////////////////////////////////////////////////////////////////
FBacktracer* FBacktracer::Instance = nullptr;

////////////////////////////////////////////////////////////////////////////////
FBacktracer::FBacktracer(FMalloc* InMalloc)
	: Malloc(InMalloc)
	, CallstackTracer(InMalloc)
{
	Instance = this;
}

////////////////////////////////////////////////////////////////////////////////
FBacktracer::~FBacktracer()
{
}

////////////////////////////////////////////////////////////////////////////////
FBacktracer* FBacktracer::Get()
{
	return Instance;
}

////////////////////////////////////////////////////////////////////////////////
#if UE_CALLSTACK_TRACE_UNIX_USE_STACK_FRAMES_WALKING
// Minimum frame count to consider frame pointer walking successful. A real
// call from the memory allocator should produce a deep stack. Fewer frames
// means frame pointers are likely not available, so fall back to backtrace().
static constexpr uint32 GMinFramesForFramePointerWalking = 5;
#endif

////////////////////////////////////////////////////////////////////////////////
uint32 FBacktracer::GetBacktraceId(void* ReturnAddress)
{
#if !UE_BUILD_SHIPPING
	uint64 StackFrames[UE_CALLSTACK_TRACE_MAX_FRAMES];
#if UE_CALLSTACK_TRACE_UNIX_USE_STACK_FRAMES_WALKING
	// Try the fast frame pointer walker first. If it doesn't return enough
	// frames (frame pointers not available), fall back to the slower but
	// more thorough DWARF-based backtrace().
	uint32 NumStackFrames = FPlatformStackWalk::CaptureStackBackTraceViaFramePointerWalking(StackFrames, UE_ARRAY_COUNT(StackFrames));
	if (NumStackFrames < GMinFramesForFramePointerWalking)
	{
		NumStackFrames = FPlatformStackWalk::CaptureStackBackTrace(StackFrames, UE_ARRAY_COUNT(StackFrames));
	}
#else
	int32 NumStackFrames = FPlatformStackWalk::CaptureStackBackTrace(StackFrames, UE_ARRAY_COUNT(StackFrames));
#endif
	if (NumStackFrames > 0)
	{
		FCallstackTracer::FBacktraceEntry BacktraceEntry;
		uint64 BacktraceId = 0;
		uint32 FrameIdx = 0;
		bool bUseAddress = false;
		for (int32 Index = 0; Index < NumStackFrames; Index++)
		{
			if (!bUseAddress)
			{
				// start using backtrace only after ReturnAddress
				if (StackFrames[Index] == (uint64)ReturnAddress)
				{
					bUseAddress = true;
				}
			}
			if (bUseAddress || NumStackFrames == 1)
			{
				uint64 RetAddr = StackFrames[Index];
				StackFrames[FrameIdx++] = RetAddr;

				// This is a simple order-dependent LCG. Should be sufficient enough
				BacktraceId += RetAddr;
				BacktraceId *= 0x30be8efa499c249dull;
			}
		}

		// Save the collected id
		BacktraceEntry.Hash = BacktraceId;
		BacktraceEntry.FrameCount = FrameIdx;
		BacktraceEntry.Frames = StackFrames;

		// Add to queue to be processed. This might block until there is room in the
		// queue (i.e. the processing thread has caught up processing).
		return CallstackTracer.AddCallstack(BacktraceEntry);
	}
#endif

	return 0;
}

////////////////////////////////////////////////////////////////////////////////
void Modules_Initialize();

////////////////////////////////////////////////////////////////////////////////
void CallstackTrace_CreateInternal(FMalloc* Malloc)
{
	if (FBacktracer::Get() != nullptr)
	{
		return;
	}

	// Allocate, construct and intentionally leak backtracer
	void* Alloc = Malloc->Malloc(sizeof(FBacktracer), alignof(FBacktracer));
	new (Alloc) FBacktracer(Malloc);
}

////////////////////////////////////////////////////////////////////////////////
void CallstackTrace_InitializeInternal()
{
	if (FBacktracer* Instance = FBacktracer::Get())
	{
#if !UE_BUILD_SHIPPING
		Modules_Initialize();
#endif
	}
}

////////////////////////////////////////////////////////////////////////////////
uint32 CallstackTrace_GetCurrentId()
{
	if (!UE_TRACE_CHANNELEXPR_IS_ENABLED(CallstackChannel))
	{
		return 0;
	}

	void* ReturnAddress = PLATFORM_RETURN_ADDRESS_FOR_CALLSTACKTRACING();
	if (FBacktracer* Instance = FBacktracer::Get())
	{
		return Instance->GetBacktraceId(ReturnAddress);
	}

	return 0;
}

#endif