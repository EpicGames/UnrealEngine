// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProfilingDebugging/MemoryTrace.h"

#if UE_MEMORY_TRACE_ENABLED

#include "CoreTypes.h"
#include "HAL/Platform.h"
#include <stdio.h>

////////////////////////////////////////////////////////////////////////////////
FMalloc* MemoryTrace_CreateInternal(FMalloc*, int, const ANSICHAR* const*);

////////////////////////////////////////////////////////////////////////////////
FMalloc* MemoryTrace_Create(FMalloc* InMalloc)
{
	if (FILE* CmdLineFile = fopen("/proc/self/cmdline", "r"))
	{
		const char* ArgV[255] = {};
		int32 ArgC = 0;

		char * Arg = nullptr;
		size_t Size = 0;
		while (getdelim(&Arg, &Size, 0, CmdLineFile) != -1)
		{
			ArgV[ArgC++] = Arg;
			Arg = nullptr; // getdelim will allocate buffer for next Arg
		}
		if (Arg != nullptr)
		{
			free(Arg);
		}
		fclose(CmdLineFile);

		FMalloc* OutMalloc = MemoryTrace_CreateInternal(InMalloc, ArgC, ArgV);

		// cleanup after getdelim
		while (ArgC > 0)
		{
			free((void*)ArgV[--ArgC]);
		}

		return OutMalloc;
	}
	return InMalloc;
}

#endif // UE_MEMORY_TRACE_ENABLED
