// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "HAL/LowLevelMemTracker.h"
#include "HAL/PlatformProcess.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/Paths.h"
#include "ProfilingDebugging/ModuleDiagnostics.h"
#include "ProfilingDebugging/MemoryTrace.h"
#include "ProfilingDebugging/MetadataTrace.h"

#if !UE_BUILD_SHIPPING
#include <link.h>
#endif

////////////////////////////////////////////////////////////////////////////////
void Modules_Initialize()
{
#if !UE_BUILD_SHIPPING
	using namespace UE::Trace;

	constexpr uint32 SizeOfSymbolFormatString = 5;
	UE_TRACE_LOG(Diagnostics, ModuleInit, ModuleChannel, sizeof(ANSICHAR) * SizeOfSymbolFormatString)
		<< ModuleInit.SymbolFormat("dwarf", SizeOfSymbolFormatString)
		<< ModuleInit.ModuleBaseShift(uint8(0));

	HeapId ProgramHeapId = MemoryTrace_HeapSpec(EMemoryTraceRootHeap::SystemMemory, TEXT("Program"), EMemoryTraceHeapFlags::NeverFrees);

	auto IterateCallback = [](struct dl_phdr_info *Info, size_t /*size*/, void *Data)
	{
		int32 TotalMemSize = 0;
		uint64 RealBase = Info->dlpi_addr;
		bool bRealBaseSet = false;

		constexpr uint32 GNUSectionNameSize = 4;
		constexpr uint32 BuildIdSize = 20;
		uint8 BuildId[BuildIdSize] = {0};
		FMemory::Memzero(BuildId, BuildIdSize);
		bool bBuildIdSet = false;
		for (int SectionIdx = 0; SectionIdx < Info->dlpi_phnum; SectionIdx++)
		{
			TotalMemSize += Info->dlpi_phdr[SectionIdx].p_memsz;

			uint64 Offset = Info->dlpi_addr + Info->dlpi_phdr[SectionIdx].p_vaddr;
			uint32 Type = Info->dlpi_phdr[SectionIdx].p_type;

			if (!bRealBaseSet && Type == PT_LOAD)
			{
				RealBase = Offset;
				bRealBaseSet = true;
			}
			if (!bBuildIdSet && Type == PT_NOTE)
			{
				ElfW(Nhdr)* Note = (ElfW(Nhdr)*)Offset;
				char* NoteName = (char*)Note + sizeof(ElfW(Nhdr));
				uint8* NoteDesc = (uint8*)Note + sizeof(ElfW(Nhdr)) + Note->n_namesz;
				if (Note->n_namesz == GNUSectionNameSize && Note->n_type == NT_GNU_BUILD_ID && strcmp(NoteName, ELF_NOTE_GNU) == 0)
				{
					FMemory::Memcpy(BuildId, NoteDesc, FMath::Min(BuildIdSize, Note->n_descsz));
					bBuildIdSet = true;
				}
			}
		}

		// Record the full path so things can be found later
		FString ImageName = FString(Info->dlpi_name);
		if (ImageName.IsEmpty())
		{
			ImageName = FPlatformProcess::ExecutablePath();
		}

		UE_TRACE_LOG(Diagnostics, ModuleLoad, ModuleChannel, sizeof(TCHAR) * ImageName.Len() + BuildIdSize)
			<< ModuleLoad.Name(*ImageName, ImageName.Len())
			<< ModuleLoad.Base(RealBase)
			<< ModuleLoad.Size(TotalMemSize)
			<< ModuleLoad.ImageId(BuildId, BuildIdSize);

#if UE_MEMORY_TRACE_ENABLED
		{
			UE_TRACE_METADATA_CLEAR_SCOPE();
			LLM(UE_MEMSCOPE(ELLMTag::ProgramSize));
			MemoryTrace_Alloc(RealBase, TotalMemSize, 1);
			HeapId ProgramHeapId = *(HeapId*)Data;
			MemoryTrace_MarkAllocAsHeap(RealBase, ProgramHeapId);
			MemoryTrace_Alloc(RealBase, TotalMemSize, 1);
		}
#endif // UE_MEMORY_TRACE_ENABLED

		return 0;
	};
	dl_iterate_phdr(IterateCallback, &ProgramHeapId);
#endif
}
