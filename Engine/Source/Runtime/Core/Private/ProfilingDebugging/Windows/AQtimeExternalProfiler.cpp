// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "ProfilingDebugging/ExternalProfiler.h"
#include "Features/IModularFeatures.h"
#include "HAL/PlatformProcess.h"
#include "Templates/UniquePtr.h"

#if UE_EXTERNAL_PROFILING_ENABLED

/**
 * AQtime implementation of FExternalProfiler
 */
class FAQtimeExternalProfiler : public FExternalProfiler
{

public:

	/** Constructor */
	FAQtimeExternalProfiler()
		: DLLHandle( NULL )
		, EnableProfiling( NULL )
	{
		// Register as a modular feature
		IModularFeatures::Get().RegisterModularFeature( FExternalProfiler::GetFeatureName(), this );
	}


	/** Destructor */
	virtual ~FAQtimeExternalProfiler()
	{
		if( DLLHandle != NULL )
		{
			FWindowsPlatformProcess::FreeDllHandle( DLLHandle );
			DLLHandle = NULL;
		}
		IModularFeatures::Get().UnregisterModularFeature( FExternalProfiler::GetFeatureName(), this );
	}

	virtual void FrameSync() override
	{

	}

	/** Gets the name of this profiler as a string.  This is used to allow the user to select this profiler in a system configuration file or on the command-line */
	virtual const TCHAR* GetProfilerName() const override
	{
		return TEXT( "AQtime" );
	}


	/** Pauses profiling. */
	virtual void ProfilerPauseFunction() override
	{
		EnableProfiling( (short)0 );
	}


	/** Resumes profiling. */
	virtual void ProfilerResumeFunction() override
	{
		EnableProfiling( (short)- 1 );
	}


	/**
	 * Initializes profiler hooks. It is not valid to call pause/ resume on an uninitialized
	 * profiler and the profiler implementation is free to assert or have other undefined
	 * behavior.
	 *
	 * @return true if successful, false otherwise.
	 */
	bool Initialize()
	{
		check( DLLHandle == NULL );

		// Try to load the VTune DLL
		DLLHandle = FWindowsPlatformProcess::GetDllHandle( TEXT( "aqProf.dll" ) );
		if( DLLHandle != NULL )
		{
			// Get API function pointers of interest
			// "EnableProfiling"
			EnableProfiling = (EnableProfilingFunctionPtr)FWindowsPlatformProcess::GetDllExport( DLLHandle, TEXT( "EnableProfiling" ) );

			if( EnableProfiling == NULL )
			{
				// Couldn't find the function we need.  AQtime support will not be active.
				FWindowsPlatformProcess::FreeDllHandle( DLLHandle );
				DLLHandle = NULL;
			}
		}

		return DLLHandle != NULL;
	}


private:

	/** Function pointer type for EnableProfiling */
	typedef void ( __stdcall *EnableProfilingFunctionPtr )( short Enable );

	/** DLL handle for VTuneApi.DLL */
	void* DLLHandle;

	/** Pointer to VTPause function. */
	EnableProfilingFunctionPtr EnableProfiling;
};


namespace AQtimeProfiler
{
	struct FAtModuleInit
	{
		FAtModuleInit()
		{
			static TUniquePtr<FAQtimeExternalProfiler> ProfilerAQtime = MakeUnique<FAQtimeExternalProfiler>();
			if( !ProfilerAQtime->Initialize() )
			{
				ProfilerAQtime.Reset();
			}
		}
	};

	static FAtModuleInit AtModuleInit;
}

#endif	// UE_EXTERNAL_PROFILING_ENABLED
