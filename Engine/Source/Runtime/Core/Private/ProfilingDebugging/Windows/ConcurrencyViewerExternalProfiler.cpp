// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "HAL/PlatformProcess.h"
#include "Misc/AssertionMacros.h"
#include "ProfilingDebugging/ExternalProfiler.h"
#include "Features/IModularFeatures.h"
#include "Templates/UniquePtr.h"
#include "Containers/Map.h"
#include "HAL/ThreadSingleton.h"

#if PLATFORM_WINDOWS && !UE_BUILD_SHIPPING && UE_EXTERNAL_PROFILING_ENABLED

bool ConcurrencyVisualizerInitialize(uint32_t MaxDepth);
void ConcurrencyVisualizerStartScopedEvent(const wchar_t* Text);
void ConcurrencyVisualizerStartScopedEventA(const char* Text);
void ConcurrencyVisualizerEndScopedEvent();

/**
 * ConcurrencyViewer implementation of FExternalProfiler
 */
class FConcurrencyViewerExternalProfiler : public FExternalProfiler
{
public:

	/** Constructor */
	FConcurrencyViewerExternalProfiler()
	{
		// Register as a modular feature
		IModularFeatures::Get().RegisterModularFeature(FExternalProfiler::GetFeatureName(), this);
	}


	/** Destructor */
	virtual ~FConcurrencyViewerExternalProfiler()
	{
		IModularFeatures::Get().UnregisterModularFeature(FExternalProfiler::GetFeatureName(), this);
	}

	virtual void FrameSync() override
	{
	}

	/** Gets the name of this profiler as a string.  This is used to allow the user to select this profiler in a system configuration file or on the command-line */
	virtual const TCHAR* GetProfilerName() const override
	{
		return TEXT("ConcurrencyViewer");
	}


	/** Pauses profiling. */
	virtual void ProfilerPauseFunction() override
	{
	}

	/** Resumes profiling. */
	virtual void ProfilerResumeFunction() override
	{
	}

	void StartScopedEvent(const struct FColor& Color, const TCHAR* Text) override
	{
		ConcurrencyVisualizerStartScopedEvent(Text);
	}

	void StartScopedEvent(const struct FColor& Color, const ANSICHAR* Text) override
	{
		ConcurrencyVisualizerStartScopedEventA(Text);
	}

	void EndScopedEvent() override
	{
		ConcurrencyVisualizerEndScopedEvent();
	}

	virtual void SetThreadName(const TCHAR* Name) override
	{
		
	}

	/**
	 * Initializes profiler hooks. It is not valid to call pause/ resume on an uninitialized
	 * profiler and the profiler implementation is free to assert or have other undefined
	 * behavior.
	 *
	 * @return true if successful, false otherwise.
	 */
	bool Initialize()
	{
		return ConcurrencyVisualizerInitialize(MaxDepth);
	}


private:
	const int32 MaxDepth = 99;
};

namespace ConcurrencyViewerProfiler
{
	struct FAtModuleInit
	{
		FAtModuleInit()
		{
			static TUniquePtr<FConcurrencyViewerExternalProfiler> Profiler = MakeUnique<FConcurrencyViewerExternalProfiler>();
			if (!Profiler->Initialize())
			{
				Profiler.Reset();
			}
		}
	};

	static FAtModuleInit AtModuleInit;
}


#endif	// UE_EXTERNAL_PROFILING_ENABLED && WITH_CONCURRENCYVIEWER_PROFILER
