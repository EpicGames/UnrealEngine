// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "ProfilingDebugging/ExternalProfiler.h"
#include "HAL/PlatformProcess.h"
#include "Features/IModularFeatures.h"
#include "Math/Color.h"
#include "Misc/Paths.h"


#if !defined(WITH_PIX_EVENT_RUNTIME)
	#define WITH_PIX_EVENT_RUNTIME 0
#endif

// Not all versions of Visual Studio include the profiler SDK headers
#if WITH_PIX_EVENT_RUNTIME && UE_EXTERNAL_PROFILING_ENABLED

// Required if we don't want the pix headers to auto detect
#define USE_PIX 1

#include "Windows/AllowWindowsPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#include "pix3.h"
THIRD_PARTY_INCLUDES_END
#include "Windows/HideWindowsPlatformTypes.h"

/**
 * PIX implementation of FExternalProfiler
 */
class FPIXExternalProfiler : public FExternalProfiler
{
public:
	FPIXExternalProfiler()
	{
		// Register as a modular feature
		IModularFeatures::Get().RegisterModularFeature(FExternalProfiler::GetFeatureName(), this);
	}

	virtual ~FPIXExternalProfiler()
	{
		if (WinPixEventRuntimeHandle)
		{
			FPlatformProcess::FreeDllHandle(WinPixEventRuntimeHandle);
			WinPixEventRuntimeHandle = nullptr;
		}
		IModularFeatures::Get().UnregisterModularFeature(FExternalProfiler::GetFeatureName(), this);
	}

	FORCEINLINE bool IsEnabled() const
	{
		return WinPixEventRuntimeHandle != nullptr;
	}

	/** Mark where the profiler should consider the frame boundary to be. */
	void FrameSync() final
	{
		/** Nothing to do here */
	}

	/** Pauses profiling. */
	void ProfilerPauseFunction() final
	{
		/** PIXEndCapture not supported on Windows */
	}

	/** Resumes profiling. */
	void ProfilerResumeFunction() final
	{
		/** PIXBeginCapture not supported on Windows */
	}

	const TCHAR* GetProfilerName() const final
	{
		return TEXT("PIX");
	}

	/** Starts a scoped event specific to the profiler. */
	void StartScopedEvent(const struct FColor& Color, const TCHAR* Text) final
	{
		if (IsEnabled())
		{
			PIXBeginEvent(Color.DWColor(), Text);
		}
	}

	/** Starts a scoped event specific to the profiler. */
	void StartScopedEvent(const struct FColor& Color, const ANSICHAR* Text) final
	{
		if (IsEnabled())
		{
			PIXBeginEvent(Color.DWColor(), Text);
		}
	}

	/** Ends a scoped event specific to the profiler. */
	void EndScopedEvent() final
	{
		if (IsEnabled())
		{
			PIXEndEvent();
		}
	}

	void SetThreadName(const TCHAR* Name) final
	{
		// Windows threads already set thread names
	}

public:
	bool Initialize()
	{
#if USE_PIX
#if PLATFORM_CPU_ARM_FAMILY && !PLATFORM_WINDOWS_ARM64EC
		FString WindowsPixDllRelativePath = FPaths::Combine(*FPaths::EngineDir(), TEXT("Binaries/ThirdParty/Windows/WinPixEventRuntime/arm64"));
		FString WindowsPixDll("WinPixEventRuntime.dll");
#else
		FString WindowsPixDllRelativePath = FPaths::Combine(*FPaths::EngineDir(), TEXT("Binaries/ThirdParty/Windows/WinPixEventRuntime/x64"));
		FString WindowsPixDll("WinPixEventRuntime.dll");
#endif
		UE_LOGF(LogProfilingDebugging, Log, "Loading %ls for PIX profiling (from %ls).", *WindowsPixDll, *WindowsPixDllRelativePath);

		WinPixEventRuntimeHandle = FPlatformProcess::GetDllHandle(*FPaths::Combine(*WindowsPixDllRelativePath, *WindowsPixDll));
		if (WinPixEventRuntimeHandle == nullptr)
		{
			const int32 ErrorNum = FPlatformMisc::GetLastError();
			TCHAR ErrorMsg[1024];
			FPlatformMisc::GetSystemErrorMessage(ErrorMsg, 1024, ErrorNum);
			UE_LOGF(LogProfilingDebugging, Error, "Failed to get %ls handle: %ls (%d)", *WindowsPixDll, ErrorMsg, ErrorNum);
		}
#endif

		return WinPixEventRuntimeHandle != nullptr;
	}

private:
	/** DLL handle for WinPixEventRuntime.dll */
	void* WinPixEventRuntimeHandle{};
};

namespace PIXProfiler
{
	struct FAtModuleInit
	{
		FAtModuleInit()
		{
			static TUniquePtr<FPIXExternalProfiler> ProfilerPIX = MakeUnique<FPIXExternalProfiler>();
			if (!ProfilerPIX->Initialize())
			{
				ProfilerPIX.Reset();
			}
		}
	};

	static FAtModuleInit AtModuleInit;
}

#endif // WITH_PIX_EVENT_RUNTIME && UE_EXTERNAL_PROFILING_ENABLED
