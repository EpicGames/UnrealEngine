// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProfilingDebugging/ExternalProfiler.h"

#if PLATFORM_WINDOWS && !UE_BUILD_SHIPPING && UE_EXTERNAL_PROFILING_ENABLED

#include "CoreTypes.h"
#include "HAL/PlatformProcess.h"
#include "Math/Color.h"
#include "Misc/AssertionMacros.h"
#include "Features/IModularFeatures.h"
#include "Templates/UniquePtr.h"

void SuperLuminalStartScopedEventWide(const wchar_t* inId, const wchar_t* inData, uint32_t inColor);
void SuperLuminalStartScopedEvent(const char* inId, const char* inData, uint32_t inColor);
void SuperLuminalEndScopedEvent();

// Use a custom color pack function here because our colors are full intensity
//  and are painful/difficult to read behind text in the app's flame graph
static uint32_t UPerformanceAPI_Pack(uint8_t R, uint8_t G, uint8_t B)
{
	const float Desaturate = 0.33f;
	float Grey = (R * 0.299f) + (G * 0.587f) + (B * 0.144f);
	R = (uint8_t)(Grey * Desaturate + ((float)R) * (1.f - Desaturate));
	G = (uint8_t)(Grey * Desaturate + ((float)G) * (1.f - Desaturate));
	B = (uint8_t)(Grey * Desaturate + ((float)B) * (1.f - Desaturate));
	return ((((uint32_t)(R)) << 24) | (((uint32_t)(G)) << 16) | (((uint32_t)(B)) << 8));
}

/**
 * Superluminal Performance Profiler implementation of FExternalProfiler
 */
class SuperluminalExternalProfiler : public FExternalProfiler
{
public:
	/** Constructor */
	SuperluminalExternalProfiler()
	{
		// Register as a modular feature
		IModularFeatures::Get().RegisterModularFeature( FExternalProfiler::GetFeatureName(), this );
	}

	/** Destructor */
	virtual ~SuperluminalExternalProfiler() {}

	/** Gets the name of this profiler as a string.  This is used to allow the user to select this profiler in a system configuration file or on the command-line */
	virtual const TCHAR* GetProfilerName() const override
	{
		return TEXT( "Superluminal" );
	}

	virtual void FrameSync() override {}
	virtual void ProfilerPauseFunction() override {}
	virtual void ProfilerResumeFunction() override {}

	/** Starts a scoped event specific to the profiler. */
	virtual void StartScopedEvent(const struct FColor& Color, const TCHAR* Text) override
	{
		SuperLuminalStartScopedEventWide(Text, nullptr, UPerformanceAPI_Pack(Color.R, Color.G, Color.B));
	}

	/** Starts a scoped event specific to the profiler. */
	virtual void StartScopedEvent(const struct FColor& Color, const ANSICHAR* Text) override
	{
		SuperLuminalStartScopedEvent(Text, nullptr, UPerformanceAPI_Pack(Color.R, Color.G, Color.B));
	}

	/** Ends a scoped event specific to the profiler. */
	virtual void EndScopedEvent() override
	{
		SuperLuminalEndScopedEvent();
	}

	/** Dead code stripper will cull our static below unless we do something. */
	virtual bool Initialize()
	{
		return true;
	}
};

namespace SuperluminalProfiler
{
	struct FAtModuleInit
	{
		FAtModuleInit()
		{
			static TUniquePtr<SuperluminalExternalProfiler> ProfilerSuperluminal = MakeUnique<SuperluminalExternalProfiler>();
			ProfilerSuperluminal->Initialize();
		}
	};

	static FAtModuleInit AtModuleInit;
}

#endif	// WITH_SUPERLUMINAL_PROFILER && UE_EXTERNAL_PROFILING_ENABLED
