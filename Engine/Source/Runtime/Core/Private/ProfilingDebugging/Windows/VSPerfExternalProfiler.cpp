// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProfilingDebugging/ExternalProfiler.h"

#if PLATFORM_WINDOWS && !UE_BUILD_SHIPPING && UE_EXTERNAL_PROFILING_ENABLED

#include "CoreTypes.h"
#include "HAL/PlatformProcess.h"
#include "Misc/AssertionMacros.h"
#include "Features/IModularFeatures.h"
#include "Templates/UniquePtr.h"

bool VSPerfInitialize();
void VSPerfDeinitialize();
bool VsPerfStartProfile();
bool VsPerfStopProfile();

/**
 * Visual Studio Profiler implementation of FExternalProfiler
 */
class FVSPerfExternalProfiler : public FExternalProfiler
{

public:

	/** Constructor */
	FVSPerfExternalProfiler()
	{
		// Register as a modular feature
		IModularFeatures::Get().RegisterModularFeature( FExternalProfiler::GetFeatureName(), this );
	}


	/** Destructor */
	virtual ~FVSPerfExternalProfiler()
	{
		VSPerfDeinitialize();
		IModularFeatures::Get().UnregisterModularFeature( FExternalProfiler::GetFeatureName(), this );
	}

	virtual void FrameSync() override
	{

	}


	/** Gets the name of this profiler as a string.  This is used to allow the user to select this profiler in a system configuration file or on the command-line */
	virtual const TCHAR* GetProfilerName() const override
	{
		return TEXT( "VSPerf" );
	}


	/** Pauses profiling. */
	virtual void ProfilerPauseFunction() override
	{
		bool Result = VsPerfStopProfile();
		if(!Result)
		{
			// ...
		}
	}


	/** Resumes profiling. */
	virtual void ProfilerResumeFunction() override
	{
		bool Result = VsPerfStartProfile();
		if(!Result)
		{
			// ...
		}
	}


	/**
	* Initializes profiler hooks. It is not valid to call pause/ Start on an uninitialized
	 * profiler and the profiler implementation is free to assert or have other undefined
	 * behavior.
	 *
	 * @return true if successful, false otherwise.
	 */
	bool Initialize()
	{
		return VSPerfInitialize();
	}
};



namespace VSPerfProfiler
{
	struct FAtModuleInit
	{
		FAtModuleInit()
		{
			static TUniquePtr<FVSPerfExternalProfiler> ProfilerVSPerf = MakeUnique<FVSPerfExternalProfiler>();
			if( !ProfilerVSPerf->Initialize() )
			{
				ProfilerVSPerf.Reset();
			}
		}
	};

	static FAtModuleInit AtModuleInit;
}


#endif	// UE_EXTERNAL_PROFILING_ENABLED
