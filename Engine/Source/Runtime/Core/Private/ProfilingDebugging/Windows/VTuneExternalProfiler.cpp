// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "HAL/PlatformProcess.h"
#include "Misc/AssertionMacros.h"
#include "ProfilingDebugging/ExternalProfiler.h"
#include "Features/IModularFeatures.h"
#include "Templates/UniquePtr.h"
#include "Containers/Map.h"
#include "HAL/ThreadSingleton.h"

#if UE_EXTERNAL_PROFILING_ENABLED

// VTune header for ITT event tracing
#include "ittnotify.h"

/** Per thread TMap for all ITT String Handles */
struct FVTunePerThreadHandleMap : public TThreadSingleton<FVTunePerThreadHandleMap>
{
	TMap<uint32, __itt_string_handle *> VTuneHandleMap;
};

/**
 * VTune implementation of FExternalProfiler
 */
class FVTuneExternalProfiler : public FExternalProfiler
{

public:

	/** Constructor */
	FVTuneExternalProfiler()
		: DLLHandle( NULL )
		, VTPause( NULL )
		, VTResume( NULL )
	{
		// Register as a modular feature
		IModularFeatures::Get().RegisterModularFeature( FExternalProfiler::GetFeatureName(), this );
	}


	/** Destructor */
	virtual ~FVTuneExternalProfiler()
	{
		if( DLLHandle != NULL )
		{
			FPlatformProcess::FreeDllHandle( DLLHandle );
			DLLHandle = NULL;
		}
		IModularFeatures::Get().UnregisterModularFeature( FExternalProfiler::GetFeatureName(), this );
	}

	virtual void FrameSync() override
	{

	}


	/** Gets the name of this profiler as a string.  This is used to allow the user to select this profiler in a system configuration file or on the command-line */
	virtual const TCHAR* GetProfilerName() const override
	{
		return TEXT( "VTune" );
	}


	/** Pauses profiling. */
	virtual void ProfilerPauseFunction() override
	{
		VTPause();
	}


	/** Resumes profiling. */
	virtual void ProfilerResumeFunction() override
	{
		VTResume();
	}

	void StartScopedEvent(const struct FColor& Color, const TCHAR* Text) override
	{
		uint32 Hash = FCrc::Strihash_DEPRECATED(Text);

		TMap<uint32, __itt_string_handle *>& VTuneHandleMap = FVTunePerThreadHandleMap::Get().VTuneHandleMap;

		if (!VTuneHandleMap.Contains(Hash))
		{
			VTuneHandleMap.Emplace(Hash, __itt_string_handle_createW(Text));
		}

		//Activate timer
		__itt_string_handle *Handle = VTuneHandleMap[Hash];
		__itt_task_begin(Domain, __itt_null, __itt_null, Handle);
	}

	void EndScopedEvent() override
	{
		//Deactivate last event
		__itt_task_end(Domain);
	}

	virtual void SetThreadName(const TCHAR* Name) override
	{
		__itt_thread_set_name(Name);
	}

	/**
	 * Initializes profiler hooks. It is not valid to call pause/ resume on an uninitialized
	 * profiler and the profiler implementation is free to assert or have other undefined
	 * behavior.
	 *
	 * @return true if successful, false otherwise.
	 */
	bool Initialize()
	{
		check( DLLHandle == NULL );

		// Try to load the VTune DLL
		DLLHandle = FPlatformProcess::GetDllHandle( TEXT( "VtuneApi.dll" ) );
		if( DLLHandle == NULL )
		{
			// 64-bit VTune Parallel Amplifier file name
			DLLHandle = FPlatformProcess::GetDllHandle( TEXT( "VtuneApi32e.dll" ) );
		}
		if( DLLHandle != NULL )
		{
			// Get API function pointers of interest
			{
				// "VTPause"
				VTPause = (VTPauseFunctionPtr)FPlatformProcess::GetDllExport( DLLHandle, TEXT( "VTPause" ) );
				if( VTPause == NULL )
				{
					// Try decorated version of this function
					VTPause = (VTPauseFunctionPtr)FPlatformProcess::GetDllExport( DLLHandle, TEXT( "_VTPause@0" ) );
				}

				// "VTResume"
				VTResume = (VTResumeFunctionPtr)FPlatformProcess::GetDllExport( DLLHandle, TEXT( "VTResume" ) );
				if( VTResume == NULL )
				{
					// Try decorated version of this function
					VTResume = (VTResumeFunctionPtr)FPlatformProcess::GetDllExport( DLLHandle, TEXT( "_VTResume@0" ) );
				}
			}

			if( VTPause == NULL || VTResume == NULL )
			{
				// Couldn't find the functions we need.  VTune support will not be active.
				FPlatformProcess::FreeDllHandle( DLLHandle );
				DLLHandle = NULL;
				VTPause = NULL;
				VTResume = NULL;
			}

			//If successfully initialized, initialize task data structures.
			if (DLLHandle != NULL)
			{
				Domain = __itt_domain_create(L"UE4Domain");
			}
		}

		return DLLHandle != NULL;
	}


private:

	/** Function pointer type for VTPause() */
	typedef void ( *VTPauseFunctionPtr )(void);

	/** Function pointer type for VTResume() */
	typedef void ( *VTResumeFunctionPtr )(void);

	/** DLL handle for VTuneApi.DLL */
	void* DLLHandle;

	/** Pointer to VTPause function. */
	VTPauseFunctionPtr VTPause;

	/** Pointer to VTResume function. */
	VTResumeFunctionPtr VTResume;

	/** Domain for ITT Events */
	__itt_domain* Domain;
};



namespace VTuneProfiler
{
	struct FAtModuleInit
	{
		FAtModuleInit()
		{
			static TUniquePtr<FVTuneExternalProfiler> ProfilerVTune = MakeUnique<FVTuneExternalProfiler>();
			if( !ProfilerVTune->Initialize() )
			{
				ProfilerVTune.Reset();
			}
		}
	};

	static FAtModuleInit AtModuleInit;
}


#endif	// UE_EXTERNAL_PROFILING_ENABLED
