// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sanitizer/RaceDetector.h"

#if USING_INSTRUMENTATION

#include "Sanitizer/RaceDetectorTypes.h"
#include "Sanitizer/RaceDetectorInterface.h"

#include "Instrumentation/Types.h"
#include "Instrumentation/Containers.h"
#include "Instrumentation/EntryPoints.h"

#include "CoreTypes.h"
#include "Trace/Trace.h"
#include "Misc/CString.h"
#include "Misc/ScopeLock.h"
#include "Misc/Paths.h"
#include "Async/Mutex.h"
#include "Async/RecursiveMutex.h"
#include "Misc/Parse.h"
#include "Math/NumericLimits.h"
#include "Misc/CommandLine.h"
#include "HAL/FileManager.h"
#include "HAL/ThreadManager.h"
#include "Tasks/Task.h"

#if PLATFORM_WINDOWS
#include "Windows/WindowsHWrapper.h"
#endif

#include <atomic>

DEFINE_LOG_CATEGORY(LogRaceDetector);

UE_TRACE_CHANNEL_DEFINE(RaceDetectorChannel, "Race condition detection for thread safety analysis. Part of the sanitizer infrastructure for detecting data races and concurrent memory access violations.");

#define WITH_RACEDETECTOR_CHECK 0
#define WITH_RACEDETECTOR_DEBUG 0

#if WITH_RACEDETECTOR_CHECK
#define RACEDETECTOR_CHECK(cond) if (!(cond)) { PLATFORM_BREAK(); }
#else
#define RACEDETECTOR_CHECK(cond)
#endif

#if WITH_RACEDETECTOR_DEBUG
UE_DISABLE_OPTIMIZATION
#endif

static bool GRaceDetectorFilterOtherThreads = false;
static bool GRaceDetectorOnlyActiveScopes = false;
static bool GRaceDetectorFilterDuplicates = true;
static FAutoConsoleVariableRef CVarReportSameRaceOnce(
	TEXT("r.RaceDetector.FilterDuplicates"),
	GRaceDetectorFilterDuplicates,
	TEXT("Whether to report the same race only once per application lifetime."),
	ECVF_Default);

static int32 GRaceDetectorActivate = 0;
static FAutoConsoleVariableRef CVarRaceDetectorActivate(
	TEXT("r.RaceDetector.Activate"),
	GRaceDetectorActivate,
	TEXT("Activate race detection for that many seconds as it most likely makes the engine non-responsive and can't be easily turned off."),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable*) INSTRUMENTATION_FUNCTION_ATTRIBUTES
		{
			UE::Sanitizer::RaceDetector::ToggleRaceDetectionUntil(UE::FTimeout(FTimespan::FromSeconds(GRaceDetectorActivate)));
		}),
	ECVF_Default);

static int32 GRaceDetectorGlobalDetailedLog = 0;
static FAutoConsoleVariableRef CVarRaceDetectorGlobalDetailedLog(
	TEXT("r.RaceDetector.GlobalDetailedLog"),
	GRaceDetectorGlobalDetailedLog,
	TEXT("Activate very detailed logging globally on all memory access."),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable*) INSTRUMENTATION_FUNCTION_ATTRIBUTES
		{
			UE::Sanitizer::RaceDetector::ToggleGlobalDetailedLog(GRaceDetectorGlobalDetailedLog);
		}),
	ECVF_Default);

static bool GRaceDetectorIgnoreRaceWithoutSecondStack = true;
static FAutoConsoleVariableRef CVarIgnoreRaceWithoutSecondStack(
	TEXT("r.RaceDetector.IgnoreRaceWithoutSecondStack"),
	GRaceDetectorIgnoreRaceWithoutSecondStack,
	TEXT("Determines if races without a second stack will still be shown.\n")
	TEXT("This generally happens when races are far enough that maybe it is not a concern after all.\n"),
	ECVF_Default);

static int32 GRaceDetectorMaxMemoryUsage = 64;
static FAutoConsoleVariableRef CVarRaceDetectorMaxMemoryUsage(
	TEXT("r.RaceDetector.MaxMemoryUsage"),
	GRaceDetectorMaxMemoryUsage,
	TEXT("How many gigabytes that the race detector is allowed to use.\n")
	TEXT("The lower the limit, the higher the chance of missing some race conditions.\n"),
	ECVF_Default);

static int32 GRaceDetectorBreakOnRace = 0;
static FAutoConsoleVariableRef CVarRaceDetectorBreakOnRace(
	TEXT("r.RaceDetector.BreakOnRace"),
	GRaceDetectorBreakOnRace,
	TEXT("Debugbreak on race detection if the debugger is attached\n")
	TEXT("	1 - Break once\n")
	TEXT("	2 - Break always\n")
	TEXT("	3 - Break only on race detected while detailed logging is active\n")
	TEXT("	4 - Break only when the second callstack is missing\n"),
	ECVF_Default);

static int32 GRaceDetectorHistoryLength = 4;
static FAutoConsoleVariableRef CVarRaceDetectorHistoryLength(
	TEXT("r.RaceDetector.HistoryLength"),
	GRaceDetectorHistoryLength,
	TEXT("Represents the number of history blocks each thread is keeping to resolve callstacks of race conditions\n")
	TEXT("Can be increased to improve detection rate for races that are very far apart\n")
	TEXT("Trying to set this below a minimum of 2 blocks won't have any effect\n"),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable*) INSTRUMENTATION_FUNCTION_ATTRIBUTES
	{
		GRaceDetectorHistoryLength = FMath::Max(2, GRaceDetectorHistoryLength);
	}),
	ECVF_Default);

using namespace UE::Instrumentation;

namespace UE::Sanitizer::RaceDetector {
	
	bool bRuntimeInitialized = false;

	UPTRINT ShadowMemoryBase = 0;
	UPTRINT ShadowMemorySize = 0;
	UPTRINT ShadowClockBase = 0;
	UPTRINT ShadowMemoryEnd = 0;
	uint32  ContextTLSIndex = UINT32_MAX;
	uint64  ContextTLSValue = 0;
	volatile uint32 GlobalEpoch = 0;
	volatile int64 FSyncObject::ObjectCount = 0;
	volatile int64 FSyncObjectBank::ObjectCount = 0;
	volatile FContextId CurrentContextId = 1;
	volatile int64 HistoryChunkCount = 0;

	static UE::FRecursiveMutex Mutex;

	FRWSpinLock RaceReportsLock;
	TArray<FString> RaceReports;

	// Lock for both ContextMapping and FreeContexts
	FRWSpinLock ContextMappingLock; 
	TSafeMap<FContextId, TRefCountPtr<FContext>> ContextMapping;
	TSafeArray<FContextId> FreeContexts;

	// Used for filtering duplicates
	TSafeSet<uint64> RaceHashes;
	FRWSpinLock RaceHashesLock;
	UE::FTimeout RaceDetectorTimeout = UE::FTimeout::Never();
	UE::FTimeout GracePeriodTimeout = UE::FTimeout::Never();
	volatile uint32 GracePeriodEpoch = 0;

	thread_local bool bIsResettingShadowThread = false;
	volatile bool bIsResettingShadow = false;
	volatile bool bIsDebuggerPresent = false;
	volatile bool bDetailedLogGlobal = false;
	volatile void* DetailedLogAddress = nullptr;

	// Can't be an atomic as it might cause infinite recursion in debug when atomics
	// are unoptimized and we end up with a function call to an instrumented function
	// going directly back to the ShouldInstrument call. std::atomics that are tested
	// after InstrumentationDepth should work but are not optimal.
	volatile uint64 RaceDetectorActive = 0;

	INSTRUMENTATION_FUNCTION_ATTRIBUTES bool ShouldInstrument(FContext& Context)
	{
		return !!RaceDetectorActive && !Context.WinInstrumentationDepth && !Context.InstrumentationDepth && !bIsResettingShadow;
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES FORCEINLINE void* GetTlsValueFast(uint32 Value)
	{
		return (void*)__readgsqword(Value);
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES FORCEINLINE FContext* GetThreadContext()
	{
		return (FContext*)GetTlsValueFast(ContextTLSValue);
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES uint64 GetSyncObjectMemoryUsage()
	{
		return Platform::SyncObjectAllocatorGetCommittedSize();
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES uint64 GetHistoryChunkMemoryUsage()
	{
		return HistoryChunkCount * sizeof(FHistoryChunk);
	}

	// ------------------------------------------------------------------------------
	// Callstack management.
	// ------------------------------------------------------------------------------
	INSTRUMENTATION_FUNCTION_ATTRIBUTES FORCEINLINE FCallstackLocation GetCurrentCallstackLocation(FContext& Context)
	{
		return FCallstackLocation(Context.CurrentCallstack, Context.CurrentCallstackSize);
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES FORCEINLINE void AddCallstackFrame(FContext& Context, void* ReturnAddress)
	{
		// History tracing needs to come first since initializing a new history chunk
		// needs to copy the current stack and we don't want the current stack
		// to contain the frame we're going to add via this function.
		if (Context.AccessHistory)
		{
			Context.AccessHistory->AddFunctionEntry(ReturnAddress);
		}

		// No need for conditionals here since we're going to generate a page fault if we 
		// go outside the context allocated memory as we are using guard pages.
		Context.CurrentCallstack[Context.CurrentCallstackSize++] = ReturnAddress;
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES FORCEINLINE void RemoveCallstackFrame(FContext& Context)
	{
		// History tracing needs to come first since initializing a new history chunk
		// needs to copy the current stack before applying the exit.
		if (Context.AccessHistory)
		{
			Context.AccessHistory->AddFunctionExit();
		}

		Context.CurrentCallstackSize--;
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES void RegisterContext(FContext& Context)
	{
		check(Context.AccessHistory == nullptr);
		Context.AccessHistory = new FAccessHistory();

		TWriteScopeLock Scope(ContextMappingLock);
		Context.GlobalEpoch = GlobalEpoch;
		if (CurrentContextId < TNumericLimits<FContextId>::Max())
		{
			Context.ContextId = CurrentContextId++;
		}
		else if (FreeContexts.Num() > 0)
		{
			// Reuse the oldest context id to have a chance to find races
			// for threads that just exited.
			Context.ContextId = FreeContexts[0];
			FreeContexts.RemoveAt(0, EAllowShrinking::No);

			// Now we can get rid of the old instrumentation context
			TRefCountPtr<FContext> OldContext;
			ContextMapping.RemoveAndCopyValue(Context.ContextId, OldContext);
			// Acquire clocks from the old context since other threads might already have entries for our context id.
			if (OldContext->GlobalEpoch == Context.GlobalEpoch)
			{
				// We acquire here since we were already given a set of vector clock from the thread that spawned us
				// and we need to union it with the context we're inheriting from.
				Context.ClockBank.Acquire(OldContext->ClockBank, _ReturnAddress());
			}

			if (Context.DetailedLogDepth || bDetailedLogGlobal)
			{
				FPlatformMisc::LowLevelOutputDebugStringf(TEXT("[%d] Recycling contextid %d from old thread %d\n"), Context.ThreadId, Context.ContextId, OldContext->ThreadId);
			}
		}
		else
		{
			UE_LOGF(LogRaceDetector, Fatal, "Too many threads active at once");
		}
		
		if (Context.DetailedLogDepth || bDetailedLogGlobal)
		{
			FPlatformMisc::LowLevelOutputDebugStringf(TEXT("[%d] Registered with contextid %d\n"), Context.ThreadId, Context.ContextId);
		}

		Context.CurrentClock() = FMath::Max(Context.StandbyClock, Context.CurrentClock());

		check(Context.ContextId != 0);
		ContextMapping.Add(Context.ContextId, &Context);
		Context.IncrementClock();
		check(Context.CurrentClock() > 0);
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES void ReleaseContext(FContext* Context)
	{
		if (Context != nullptr)
		{
			if (Context->InstrumentationDepth != 0)
			{
				UE_LOGF(LogRaceDetector, Fatal, "Trying to release an instrumentation context still in use");
			}

			// Check if the context has ever been registered
			if (Context->ContextId)
			{
				// Backup the current clock as we're releasing our context id and won't have access to the clock afterward.
				Context->StandbyClock = Context->CurrentClock();
				// We don't destroy the context here as we want to leave a chance
				// to find race in short lived threads or threads that race just
				// before exiting. We'll recycle it when its ContextId gets reused.
				TWriteScopeLock Scope(ContextMappingLock);
				FreeContexts.Push(Context->ContextId);
				Context->ContextId = 0;
			}
		}
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES TRefCountPtr<FContext> GetContextById(FContextId ContextId)
	{
		TWriteScopeLock Scope(ContextMappingLock);
		return ContextMapping.FindRef(ContextId);
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES TRefCountPtr<FContext> GetContextByThreadId(uint32 ThreadId)
	{
		TWriteScopeLock Scope(ContextMappingLock);
		for (auto& [Key, Context] : ContextMapping)
		{
			if (Context->ThreadId == ThreadId)
			{
				return Context;
			}
		}
		return nullptr;
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES FORCEINLINE UPTRINT GetAlignedAddress(UPTRINT Ptr)
	{
		return UPTRINT(Ptr) & ~UPTRINT(0b111);
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES FORCEINLINE void* GetAlignedPointer(void* Ptr)
	{
		return (void*)GetAlignedAddress((UPTRINT)Ptr);
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES FORCEINLINE UPTRINT GetShadowMemoryAddress(UPTRINT Ptr)
	{
		static_assert(sizeof(FShadowMemory) <= sizeof(UPTRINT) * 4);

		// Map the higher address space as a continuation of the lower one by just removing the shadow size from it.
		if (Ptr >= ShadowMemoryEnd)
		{
			Ptr -= ShadowMemorySize;
		}

		UPTRINT Result = ShadowMemoryBase + (Ptr >> 3) * sizeof(FShadowMemory);
		RACEDETECTOR_CHECK(Result >= ShadowMemoryBase && Result < ShadowClockBase);
		return Result;
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES FShadowMemory* GetShadowMemory(UPTRINT Ptr)
	{
		UPTRINT ShadowMemoryAddress = GetShadowMemoryAddress(Ptr);
		// When the debugger is present, we map shadow memory before using it since its the fastest method.
		// If the debugger is not present we let the page fault handler take care of page faults,
		// because its even faster (i.e. less memory lookups / cache misses).
		if (bIsDebuggerPresent)
		{
			Platform::MapShadowMemory(ShadowMemoryAddress, sizeof(FShadowMemory));
		}
		return (FShadowMemory*)ShadowMemoryAddress;
	}
	
	INSTRUMENTATION_FUNCTION_ATTRIBUTES FORCEINLINE UPTRINT GetShadowClockBankMemoryAddress(UPTRINT Ptr)
	{
		static_assert(sizeof(FShadowClockBankSlot) <= sizeof(UPTRINT));

		// Map the higher address space as a continuation of the lower one by just removing the shadow size from it.
		if (Ptr >= ShadowMemoryEnd)
		{
			Ptr -= ShadowMemorySize;
		}

		UPTRINT Result = ShadowClockBase + (Ptr >> 3) * sizeof(FShadowClockBankSlot);
		RACEDETECTOR_CHECK(Result >= ShadowClockBase && Result < ShadowMemoryEnd);
		return Result;
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES FShadowClockBankSlot* GetShadowClockBank(UPTRINT Ptr)
	{
		UPTRINT ShadowMemoryAddress = GetShadowClockBankMemoryAddress(Ptr);
		
		// Always do mapping to avoid reentrancy in the vectored exception handler that
		// uses SRW lock and end up calling back into this function for another sync object.
		Platform::MapShadowMemory(ShadowMemoryAddress, sizeof(FShadowClockBankSlot));

		return (FShadowClockBankSlot*)ShadowMemoryAddress;
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES FSyncObjectRef GetSyncObject(FContext& Context, void* SyncAddr)
	{
		FShadowClockBankSlot* ClockBankSlot = GetShadowClockBank((UPTRINT)SyncAddr);
		const UPTRINT Index = UPTRINT(SyncAddr) & 7;
		RACEDETECTOR_CHECK(Context.BankHazard == nullptr);

		// This can race with ResetShadow so use Hazard Pointer mechanism to avoid
		// the object bank from being decommitted while we're creating an FSyncObjectRef to it.
		while (true)
		{
			FSyncObjectBank* LocalBank = ClockBankSlot->SyncObjectBank;
			if (LocalBank == nullptr)
			{
				FSyncObjectBank* NewObjectBank = new FSyncObjectBank();

				FSyncObjectBank* OldObjectBank = (FSyncObjectBank*)FPlatformAtomics::InterlockedCompareExchangePointer((void**)&ClockBankSlot->SyncObjectBank, NewObjectBank, nullptr);
				if (OldObjectBank == nullptr)
				{
					// We published successfully. No hazard pointer handling needed since
					// we own the bank and nobody else can see it yet.
					LocalBank = NewObjectBank;
					FSyncObjectRef Result = FSyncObjectRef(LocalBank, LocalBank->GetSyncObject(Index));
					Context.BankHazard = nullptr;
					return MoveTemp(Result);
				}

				// Another thread won the race — our bank is wasted but this is very rare.
				// The linear allocator will reclaim it on the next reset.
				LocalBank = OldObjectBank;
			}

			Context.BankHazard = LocalBank;
			Platform::AsymmetricThreadFenceLight();

			// Confirm that the entry is still in the shadow slot, otherwise we iterate again as our
			// hazard pointer protection wouldn't be guaranteed.
			if (LocalBank == ClockBankSlot->SyncObjectBank)
			{
				FSyncObjectRef Result = FSyncObjectRef(LocalBank, LocalBank->GetSyncObject(Index));
				Context.BankHazard = nullptr;
				return MoveTemp(Result);
			}
		}
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES void FreeMemoryRange(void* Ptr, uint64 Size)
	{
		if (!RaceDetectorActive || bIsResettingShadow)
		{
			return;
		}

		UPTRINT BankAddr = UPTRINT(Ptr) >> 3;
		UPTRINT ShadowAddr = GetShadowMemoryAddress((UPTRINT)Ptr);
		for (uint64 Index = 0; Index < Size; Index += 8, ++BankAddr, ShadowAddr += sizeof(FShadowMemory))
		{
			// We only need to verify this once in every PageSize so we could optimize this further if needed.
			if (Platform::IsShadowMemoryMapped(ShadowAddr, sizeof(FShadowMemory)))
			{
				FShadowMemory* Shadow = (FShadowMemory*)ShadowAddr;

				Shadow->Accesses[0].RawValue = 0;
				Shadow->Accesses[1].RawValue = 0;
				Shadow->Accesses[2].RawValue = 0;
				Shadow->Accesses[3].RawValue = 0;
			}
		}
	}
	
	INSTRUMENTATION_FUNCTION_ATTRIBUTES FORCEINLINE uint8 GetOffsetInBytes(void* Ptr)
	{
		UPTRINT OriginalAddr = (UPTRINT)Ptr;
		UPTRINT FinalAddr = OriginalAddr & ~UPTRINT(0b111);
		return (uint8)(OriginalAddr - FinalAddr);
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES FORCEINLINE void ResetShadowMemory()
	{
		Platform::UnmapShadowMemory();
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES void WriteToLog(FString&& Msg)
	{
		TWriteScopeLock Lock(RaceReportsLock);
		RaceReports.Emplace(MoveTemp(Msg));
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES FHistoryChunk::FHistoryChunk()
	{
		FPlatformAtomics::InterlockedIncrement(&HistoryChunkCount);
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES void FHistoryChunk::InitStack()
	{
		FContext* ContextPtr = (FContext*)GetTlsValueFast(ContextTLSValue);
		if (FContext::IsValid(ContextPtr))
		{
			StartClock = ContextPtr->CurrentClock();
			for (uint16 Index = 0; Index < ContextPtr->CurrentCallstackSize; ++Index)
			{
				new (Buffer + Offset) FHistoryEntryFunctionEntry(ContextPtr->CurrentCallstack[Index]);
				Offset += sizeof(FHistoryEntryFunctionEntry);
			}
		}
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES FHistoryChunk::~FHistoryChunk()
	{
		FPlatformAtomics::InterlockedDecrement(&HistoryChunkCount);
	}

	TOptional<TRaceCallbackFn> RaceCallbackFn;
	INSTRUMENTATION_FUNCTION_ATTRIBUTES void ReportRace(FContext& Context, void* Pointer, const FMemoryAccess& CurrentAccess, const FMemoryAccess& RacingAccess)
	{
		TRefCountPtr<FContext> RacingContext = GetContextById(RacingAccess.ContextId);
		// The thread might have been destroyed
		if (!RacingContext.IsValid())
		{ 
			return;
		}

		FCallstackLocation FirstLocation = GetCurrentCallstackLocation(Context);

		const uint32 FirstThreadId = Context.ThreadId;
		const uint32 SecondThreadId = RacingContext->ThreadId;

		// Dumb scan of the history to find the matching race so we can get its location.
		FCallstackLocation SecondLocation;
		const void* AlignedPointer = GetAlignedPointer(Pointer);
		FAccessHistory& SecondAccessHistory = *RacingContext->AccessHistory;
		FClockRange HistoryRange;
		bool bFoundLocation = SecondAccessHistory.ResolveAccess(Context, AlignedPointer, RacingAccess, SecondLocation, HistoryRange);

		// Resolving symbols can cause us to enter a wait that could try to start a new thread
		// and wait for it to be started. We can't allow that since we could deadlock
		// if we're reporting a race while having a lock that the new thread might also need
		// during its initialization. (i.e. Registering new FNames)
		LowLevelTasks::Private::FOversubscriptionAllowedScope AllowOversubscription(false);

		if (RaceCallbackFn)
		{
			FFullLocation FirstFullLocation = FirstLocation.GetFullLocation();
			FFullLocation SecondFullLocation = SecondLocation.GetFullLocation();
			RaceCallbackFn.GetValue()((UPTRINT)Pointer, FirstThreadId, SecondThreadId, FirstFullLocation, SecondFullLocation);
		}
		else
		{
			if (!bFoundLocation && GRaceDetectorIgnoreRaceWithoutSecondStack)
			{
				return;
			}

			if (GRaceDetectorFilterDuplicates)
			{
				uint64 FirstLocationLastFrame = FirstLocation.GetLastFrame();
				uint64 SecondLocationLastFrame = SecondLocation.GetLastFrame();
				uint64 LocationHash = FirstLocationLastFrame ^ SecondLocationLastFrame;
				bool bAlreadyFound = false;
				{
					TWriteScopeLock Lock(RaceHashesLock);
					RaceHashes.FindOrAdd(LocationHash, &bAlreadyFound);
				}
				if (bAlreadyFound)
				{
					return;
				}
			}

			if (GRaceDetectorFilterOtherThreads &&
				Context.bAlwaysReport != true &&
				RacingAccess.bAlwaysReport != true)
			{
				return;
			}

			void* LowLimit = 0; void *HighLimit = 0;
			Platform::GetCurrentThreadStackLimits(&LowLimit, &HighLimit);
			bool bIsRaceOnStack = false;
			if (Pointer >= LowLimit && Pointer < HighLimit)
			{
				bIsRaceOnStack = true;
			}

			FFullLocation FirstFullLocation = FirstLocation.GetFullLocation();
			FFullLocation SecondFullLocation = SecondLocation.GetFullLocation();

			FString FirstThreadName;
			FString SecondThreadName;
			bool bNeedResolve = true;
			FThreadManager::Get().ForEachThread(
				[&](uint32, FRunnableThread*)
				{
					// GetThreadName is not thread-safe when a thread is exiting since it just sends us a reference to the string
					// which can be deleted once the lock is not held. So resolve the names while inside ForEachThread
					// since it maintains the lock while we copy the thread names. This is just a workaround until GetThreadName 
					// can be fixed.
					if (bNeedResolve)
					{
						FirstThreadName = FThreadManager::Get().GetThreadName(FirstThreadId);
						SecondThreadName = FThreadManager::Get().GetThreadName(SecondThreadId);
						bNeedResolve = false;
					}
				}
			);

			FString NotFoundMessage;
			if (!bFoundLocation)
			{
				NotFoundMessage = FString::Printf(
					TEXT("Location not found in access history.\n")
					TEXT("Number of history blocks recycled: %llu\n")
					TEXT("History range: clock %d to %d\n")
					TEXT("Last recycle: %.02f seconds ago"),
					SecondAccessHistory.RecycleCount,
					HistoryRange.First, HistoryRange.Last,
					FPlatformTime::Seconds() - SecondAccessHistory.LastRecycle
				);
			}
			
			FClock LastSyncClock = Context.ClockBank.Get(RacingAccess.ContextId);
			FFullLocation LastSyncLocation = Context.ClockBank.GetLocation(RacingAccess.ContextId).GetFullLocation();

			const int32 Alignment = FMath::Max3(FirstFullLocation.GetAlignment(), LastSyncLocation.GetAlignment(), SecondFullLocation.GetAlignment());

			TStringBuilder<4096> Report;
			Report.Appendf(
				TEXT("=========================================") LINE_TERMINATOR
				TEXT("WARNING: RaceDetector: data race detected") LINE_TERMINATOR
				TEXT("%s%s of size %d at %p %smade at clock %lu by thread %s (%d) which is now at clock %lu:") LINE_TERMINATOR 
				TEXT("%s") LINE_TERMINATOR
				TEXT("Previous %s%s of size %d at %p at clock %lu by thread %s (%d %s) which is now at clock %lu:") LINE_TERMINATOR
				TEXT("%s") LINE_TERMINATOR
				TEXT("Last known sync clock between both threads is %lu:") LINE_TERMINATOR
				TEXT("%s") LINE_TERMINATOR
				TEXT("=========================================") LINE_TERMINATOR,
				AccessTypeToString(CurrentAccess.AccessType),
				(CurrentAccess.AccessType & EMemoryAccessType::ACCESS_TYPE_VPTR) ? TEXT(" (vptr)") : TEXT(""),
				CurrentAccess.GetSize(),
				(void*)((UPTRINT)AlignedPointer + CurrentAccess.GetOffset()),
				bIsRaceOnStack ? TEXT("(Stack) ") : TEXT(""),
				CurrentAccess.Clock,
				*FirstThreadName,
				FirstThreadId,
				Context.CurrentClock(),
				*FirstFullLocation.ToString(Alignment),
				AccessTypeToString(RacingAccess.AccessType),
				(RacingAccess.AccessType& EMemoryAccessType::ACCESS_TYPE_VPTR) ? TEXT(" (vptr)") : TEXT(""),
				RacingAccess.GetSize(),
				(void*)((UPTRINT)AlignedPointer + RacingAccess.GetOffset()),
				RacingAccess.Clock,
				*SecondThreadName,
				SecondThreadId,
				Platform::IsThreadAlive(SecondThreadId) ? TEXT("alive") : TEXT("exited"),
				RacingContext->CurrentClock(),
				bFoundLocation ? *SecondFullLocation.ToString(Alignment) : *NotFoundMessage,
				LastSyncClock,
				*LastSyncLocation.ToString(Alignment)
			);

			FPlatformMisc::LowLevelOutputDebugString(Report.ToString());

			// We do as little as possible from within the race since it might cause reentrancy depending on where the race happens.
			// Send the report to be properly logged to file in another thread.
			WriteToLog(Report.ToString());
		}
		
		if (GRaceDetectorBreakOnRace && FPlatformMisc::IsDebuggerPresent())
		{
			if ((GRaceDetectorBreakOnRace == 3 && !Context.DetailedLogDepth) ||
				(GRaceDetectorBreakOnRace == 4 && bFoundLocation))
			{
				return;
			}

			// Reset for break once.
			if (GRaceDetectorBreakOnRace == 1)
			{
				GRaceDetectorBreakOnRace = 0;
			}

			PLATFORM_BREAK();
		}
	}

	INSTRUMENTATION_ENTRYPOINT_ATTRIBUTES void SetRaceCallbackFn(TRaceCallbackFn CallbackFn)
	{
		RaceCallbackFn = CallbackFn;
	}

	INSTRUMENTATION_ENTRYPOINT_ATTRIBUTES void ResetRaceCallbackFn()
	{
		RaceCallbackFn.Reset();
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES FORCEINLINE void InstrumentAccess(FContext& Context, void* Ptr, uint8 Size, EMemoryAccessType AccessType, FAtomicMemoryOrder Order, void* ReturnAddress, const TCHAR* OpName, bool& bHasAddedStack)
	{
		// Validate size and alignment since it should have been done before entering this function.
		RACEDETECTOR_CHECK(Size <= 8 && GetShadowMemoryAddress((UPTRINT)Ptr) == GetShadowMemoryAddress((UPTRINT)Ptr + Size - 1));

		// GetShadowMemory will contain the cost of committing pages to shadow memory
		// The cost of this is higher when run under the debugger because we're not
		// using vectored exception handler in this case due to abysmal perf caused
		// by the kernel sending exception events to the attached debugger.
		FShadowMemory* Shadow = GetShadowMemory((UPTRINT)Ptr);

		// Use short variable name here to improve readability further down.
		FMemoryAccess* S = Shadow->Accesses;

		// Try to hide cache miss on shadow memory behind some computational stuff.
		FPlatformMisc::PrefetchBlock(S, sizeof(Shadow->Accesses));

		// Grab a ContextId on first instrumented access to avoid wasting our precious 256 available context
		// on thirdparty and OS threads that never enter into instrumented code.
		if (Context.ContextId == 0)
		{
			RegisterContext(Context);
		}

		const uint32 Epoch = GlobalEpoch;
		const uint8 Offset = GetOffsetInBytes(Ptr);
		// Store the bAlwaysReport of this context directly in the access bitfield because at the time of the race, the racingcontext might have a different bAlwaysReport state than when the access was made.
		// So to get consistent results, it's important to store the state of the context as it was when the access was made.
		const FMemoryAccess CurrentAccess(Context.ContextId, Context.CurrentClock(), Offset, Size, AccessType, Context.bAlwaysReport);

		// This is very practical to scope small section of code that we want to understand
		// exactly what's going on inside.
		if (UNLIKELY(Context.DetailedLogDepth || bDetailedLogGlobal))
		{
			if (DetailedLogAddress == nullptr ||
				DetailedLogAddress == Ptr)
			{
				FPlatformMisc::LowLevelOutputDebugStringf(
					TEXT("[%d] %s / %s of size %d offset %d (%s) at %p (ctx:%d clk:%u)")
					TEXT(" S0 a:%d s:%d o:%d ctx:%d clk:%u")
					TEXT(" S1 a:%d s:%d o:%d ctx:%d clk:%u")
					TEXT(" S2 a:%d s:%d o:%d ctx:%d clk:%u")
					TEXT(" S3 a:%d s:%d o:%d ctx:%d clk:%u\n"),
					Context.ThreadId,
					OpName,
					AccessTypeToString(AccessType),
					Size,
					Offset,
					LexToString(Order),
					Ptr,
					CurrentAccess.ContextId,
					CurrentAccess.Clock,
					S[0].AccessType, S[0].GetSize(), S[0].GetOffset(), S[0].ContextId, S[0].Clock,
					S[1].AccessType, S[1].GetSize(), S[1].GetOffset(), S[1].ContextId, S[1].Clock,
					S[2].AccessType, S[2].GetSize(), S[2].GetOffset(), S[2].ContextId, S[2].Clock,
					S[3].AccessType, S[3].GetSize(), S[3].GetOffset(), S[3].ContextId, S[3].Clock
				);
			}
		}

		// The cost of this is going to be higher when we're not running under the debugger
		// because the vectored page fault handler is called as part of this access on new shadow memory.
		if (S[0].RawValue == CurrentAccess.RawValue ||
			S[1].RawValue == CurrentAccess.RawValue ||
			S[2].RawValue == CurrentAccess.RawValue ||
			S[3].RawValue == CurrentAccess.RawValue)
		{
			// Skip everything including history logging if our access is already in the shadow.
			return;
		}

		if (!bHasAddedStack)
		{
			AddCallstackFrame(Context, ReturnAddress);
			bHasAddedStack = true;
		}

		// Write to our history first so that any race detected from the shadow
		// can be resolved.
		Context.AccessHistory->AddMemoryAccess(GetAlignedPointer(Ptr), CurrentAccess);

		bool bSaveNeeded = true;
		FMemoryAccess PreviousAccess;
		// We don't care about ordering but each 64-bit needs to keep their integrity.
		// This will remain thread-safe as long as we are only reading and writing whole 64-bit words.
		for (int32 Index = 0; Index < 4; ++Index)
		{
			uint64& RawValue = Shadow->Accesses[Index].RawValue;

			// Read the whole 64-bit and store it locally to keep things atomic and thread-safe.
			PreviousAccess.RawValue = RawValue;

			// We fill the slots in order so we can early out.
			// We can assume other slots are empty if we find one empty.
			if (PreviousAccess.AccessType == EMemoryAccessType::ACCESS_TYPE_INVALID)
			{
				if (bSaveNeeded)
				{
					// Overwrite the whole 64-bit word to keep things atomic and thread-safe.
					RawValue = CurrentAccess.RawValue;
				}
				return;
			}

			// Should never happen to have identical values since we verified at entry.
			RACEDETECTOR_CHECK(PreviousAccess.RawValue != CurrentAccess.RawValue);

			// Check if there is any overlap with the previous access
			// Each bit in Access represets 1-byte in memory so we can just
			// AND both access to know if we have common bytes being accessed.
			if (!(CurrentAccess.Access & PreviousAccess.Access))
			{
				continue;
			}

			// If we already have a slot, upgrade it if possible to avoid spilling to too many slots.
			if (PreviousAccess.ContextId == CurrentAccess.ContextId)
			{
				// Most recent clock first, then what we want in the slot is what is most susceptible to cause a race.
				// We try to keep non-atomic first, then write first.. then finally reads.
				if (bSaveNeeded &&
					(CurrentAccess.Clock > PreviousAccess.Clock ||
					 PreviousAccess.bIsAtomic > CurrentAccess.bIsAtomic ||
					 CurrentAccess.bIsWrite > PreviousAccess.bIsWrite))
				{
					RawValue = CurrentAccess.RawValue;
					bSaveNeeded = false;
				}
				continue;
			}

			// Check for obvious correct case where we're not racing.
			const bool bBothRead   = !(PreviousAccess.bIsWrite | CurrentAccess.bIsWrite);
			const bool bBothAtomic = PreviousAccess.bIsAtomic & CurrentAccess.bIsAtomic;
			if (bBothRead | bBothAtomic)
			{
				continue;
			}

			// Verify that the clock we got in our bank for the previous access context has been
			// synchronized since the last access... if not it means a barrier is missing and we have a race.
			FClock Clock = Context.ClockBank.Get(PreviousAccess.ContextId);
			if (Clock >= PreviousAccess.Clock)
			{
				continue;
			}

			// Reset the shadow to avoid reporting the same race multiple times
			// Use the first reset as a sync point between multiple threads that might want to report the same race.
			bool bWonReportingRace = FPlatformAtomics::InterlockedExchange((volatile int64*)&Shadow->Accesses[0].RawValue, 0);
			Shadow->Accesses[1].RawValue = 0;
			Shadow->Accesses[2].RawValue = 0;
			Shadow->Accesses[3].RawValue = 0;

			// We can find false positives when the race detector is shutting down or during shadow resets
			// because some function will stop instrumenting before others, etc...
			// Just ignore any race we have found if anything seems incoherent.
			if (bWonReportingRace && RaceDetectorActive && Epoch == GlobalEpoch && !bIsResettingShadow)
			{
				ReportRace(Context, Ptr, CurrentAccess, PreviousAccess);
			}

			// Just return now as we're in the race reporting case where we don't need to save our access anymore.
			return;
		}

		if (bSaveNeeded)
		{
			// We haven't saved our access yet.
			// Use history index as a 'random' position to avoid expensive computations.
			Shadow->Accesses[Context.AccessHistory->GetOffset() & 3].RawValue = CurrentAccess.RawValue;
		}
	}

	// Handles alignment and sizes bigger than a shadow cell
	INSTRUMENTATION_FUNCTION_ATTRIBUTES FORCEINLINE void InstrumentAccessPreamble(FContext& Context, UPTRINT Ptr, uint32 Size, EMemoryAccessType AccessType, FAtomicMemoryOrder Order, void* ReturnAddress, const TCHAR* OpName, bool& bHasAddedCallstackFrame)
	{
		UPTRINT AdjustedSize = FMath::Min((UPTRINT)Size, GetAlignedAddress(Ptr) + 8 - Ptr);
		InstrumentAccess(Context, (void*)Ptr, static_cast<uint8>(AdjustedSize), AccessType, Order, ReturnAddress, OpName, bHasAddedCallstackFrame);
		Ptr += AdjustedSize;
		Size -= AdjustedSize;

		if (Size)
		{
			while (Size >= 8)
			{
				InstrumentAccess(Context, (void*)Ptr, static_cast<uint8>(8), AccessType, Order, ReturnAddress, OpName, bHasAddedCallstackFrame);
				Ptr += 8;
				Size -= 8;
			}

			if (Size)
			{
				InstrumentAccess(Context, (void*)Ptr, static_cast<uint8>(Size), AccessType, Order, ReturnAddress, OpName, bHasAddedCallstackFrame);
			}
		}
	}

	template <typename AtomicOpType>
	INSTRUMENTATION_FUNCTION_ATTRIBUTES void InstrumentAtomicAccess(FContext& Context, void* AtomicAddr, uint8 Size, EMemoryAccessType AccessType, FAtomicMemoryOrder Order, AtomicOpType&& AtomicOp, void* ReturnAddress, const TCHAR* OpName, bool& bHasAddedCallstackFrame)
	{
		RACEDETECTOR_CHECK(FMath::Min((UPTRINT)Size, GetAlignedAddress((UPTRINT)AtomicAddr) + 8 - (UPTRINT)AtomicAddr) == (UPTRINT)Size);

		InstrumentAccess(Context, AtomicAddr, Size, AccessType, Order, ReturnAddress, OpName, bHasAddedCallstackFrame);
		if (IsAtomicOrderRelaxed(Order))
		{
			AtomicOp();
			return;
		}

		FSyncObjectRef Atomic = GetSyncObject(Context, AtomicAddr);

		if (AccessType == ACCESS_TYPE_ATOMIC_READ_WRITE && IsAtomicOrderAcquireRelease(Order))
		{
			Atomic->SyncAcquireRelease(Context, AtomicOp, ReturnAddress, AtomicAddr, OpName);
		}
		else if ((AccessType & ACCESS_TYPE_ATOMIC_READ) == ACCESS_TYPE_ATOMIC_READ && IsAtomicOrderAcquire(Order))
		{
			Atomic->SyncAcquire(Context, AtomicOp, ReturnAddress, AtomicAddr, OpName);
		}
		else if ((AccessType & ACCESS_TYPE_ATOMIC_WRITE) == ACCESS_TYPE_ATOMIC_WRITE && IsAtomicOrderRelease(Order))
		{
			Atomic->SyncRelease(Context, AtomicOp, ReturnAddress, AtomicAddr, OpName);
		}
		else
		{
			checkf(false, TEXT("Unexpected memory order"));
		}

		Context.IncrementClock();
	}

	template <typename AtomicOpType>
	INSTRUMENTATION_FUNCTION_ATTRIBUTES void InstrumentAtomicAccess(FContext& Context, void* AtomicAddr, uint8 Size, EMemoryAccessType AccessType, FAtomicMemoryOrder SuccessOrder, FAtomicMemoryOrder FailureOrder, AtomicOpType&& AtomicOp, void* ReturnAddress, const TCHAR* OpName, bool& bHasAddedCallstackFrame)
	{
		RACEDETECTOR_CHECK(FMath::Min((UPTRINT)Size, GetAlignedAddress((UPTRINT)AtomicAddr) + 8 - (UPTRINT)AtomicAddr) == (UPTRINT)Size);

		if (IsAtomicOrderRelaxed(SuccessOrder) && IsAtomicOrderRelaxed(FailureOrder))
		{
			// Both orders are relaxed, forward either one of them.
			InstrumentAccess(Context, AtomicAddr, Size, AccessType, SuccessOrder, ReturnAddress, OpName, bHasAddedCallstackFrame);
			
			AtomicOp();
		}
		else
		{
			GetSyncObject(Context, AtomicAddr)->SyncWithFailureSupport(Context, AtomicOp, AccessType, SuccessOrder, FailureOrder, ReturnAddress, AtomicAddr, OpName,
				[&Context, AtomicAddr, Size, AccessType, ReturnAddress, &SuccessOrder, OpName, &bHasAddedCallstackFrame](FAtomicMemoryOrder ActualOrder) INSTRUMENTATION_FUNCTION_ATTRIBUTES
				{
					InstrumentAccess(Context, AtomicAddr, Size, AccessType, ActualOrder, ReturnAddress, OpName, bHasAddedCallstackFrame);
				}
			);
		}
	}

	INSTRUMENTATION_ENTRYPOINT_ATTRIBUTES void DumpContext()
	{
		FInstrumentationScope InstrumentationScope;
		FContext& Context = EnsureCurrentContext();

		TStringBuilder<4096> OtherClocks;
		for (FContextId ContextId = 0; ContextId < 256; ++ContextId)
		{
			if (Context.ClockBank.Get(ContextId) != 0)
			{
				OtherClocks.Appendf(TEXT("[%d=%d]"), ContextId, Context.ClockBank.Get(ContextId));
			}
		}

		FPlatformMisc::LowLevelOutputDebugStringf(
			TEXT("Thread %s (%d), context %d, clock %d, other clocks %s\n"), 
			*FThreadManager::Get().GetThreadName(Context.ThreadId),
			Context.ThreadId, 
			Context.ContextId, 
			Context.CurrentClock(), 
			*OtherClocks.ToString());
	}

	INSTRUMENTATION_ENTRYPOINT_ATTRIBUTES void DumpContextDetailed()
	{
		FInstrumentationScope InstrumentationScope;
		FContext& Context = EnsureCurrentContext();

		FPlatformMisc::LowLevelOutputDebugStringf(
			TEXT("Thread %s (%d), context %d, clock %d\n"),
			*FThreadManager::Get().GetThreadName(Context.ThreadId),
			Context.ThreadId, 
			Context.ContextId,
			Context.CurrentClock()
		);

		for (FContextId ContextId = 0; ContextId < 256; ++ContextId)
		{
			if (Context.ClockBank.Get(ContextId) != 0)
			{
				FPlatformMisc::LowLevelOutputDebugStringf(
					TEXT("  [%d=%d]\n%s\n"), 
					ContextId, 
					Context.ClockBank.Get(ContextId),
					*Context.ClockBank.GetLocation(ContextId).GetFullLocation().ToString()
				);
			}
		}
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES void UnhookInstrumentation();
	INSTRUMENTATION_FUNCTION_ATTRIBUTES void HookInstrumentation();
		
	INSTRUMENTATION_FUNCTION_ATTRIBUTES TSet<FSyncObjectBank*> GatherBankHazards()
	{
		// Make sure all threads have published their BankHazard
		Platform::AsymmetricThreadFenceHeavy();

		TSet<FSyncObjectBank*> Hazards;
		TReadScopeLock Scope(ContextMappingLock);
		for (auto& Pair : ContextMapping)
		{
			if (FSyncObjectBank* BankHazard = Pair.Value->BankHazard)
			{
				Hazards.Add(BankHazard);
			}
		}
		return Hazards;
	}
	

	INSTRUMENTATION_FUNCTION_ATTRIBUTES void ResetShadow()
	{
		static UE::FMutex Lock;
		if (!Lock.TryLock())
		{
			FPlatformMisc::LowLevelOutputDebugStringf(TEXT("[%d] Skip duplicate reset shadow\n"), Platform::GetCurrentThreadId());
			return;
		}
		ON_SCOPE_EXIT
		{
			Lock.Unlock();
		};

		// Need to take the mutex during reset shadow since we're unhooking and rehooking instrumentation, which is not thread-safe.
		UE::TUniqueLock ScopeLock(Mutex);

		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("[%d] Reset Shadow Memory Started\n"), Platform::GetCurrentThreadId());

		FPlatformAtomics::InterlockedExchange((volatile int8*)&bIsResettingShadow, (int8)1);
		bIsResettingShadowThread = true;

		if (RaceDetectorActive)
		{
			UnhookInstrumentation();
		}

		ResetShadowMemory();

		// Wait for all hazard pointers to clear (threads finishing GetSyncObject).
		// Since instrumentation is stopped, no new hazards will be created.
		while (true)
		{
			TSet<FSyncObjectBank*> Hazards = GatherBankHazards();
			if (Hazards.Num() == 0)
			{
				break;
			}
			FPlatformMisc::LowLevelOutputDebugStringf(TEXT("[%d] Waiting for %d hazard pointers to clear\n"), Platform::GetCurrentThreadId(), Hazards.Num());
			FPlatformProcess::Yield();
		}

		// Wait for all outstanding FSyncObjectRef instances to be released.
		// These are very short-lived (stack-scoped in InstrumentAtomicAccess),
		// so this should drain almost immediately after instrumentation is stopped.
		while (Platform::SyncObjectAllocatorGetRefCount() > 0)
		{
			FPlatformMisc::LowLevelOutputDebugStringf(TEXT("[%d] Waiting for %d sync object refs to drain\n"), Platform::GetCurrentThreadId(), Platform::SyncObjectAllocatorGetRefCount());
			FPlatformProcess::Yield();
		}

		// Decommit all sync object memory in a single syscall
		Platform::SyncObjectAllocatorReset();
		FSyncObject::ResetObjectCount();
		FSyncObjectBank::ResetObjectCount();

		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("[%d] Reset Shadow Memory Ended\n"), Platform::GetCurrentThreadId());
		
		if (RaceDetectorActive)
		{
			GlobalEpoch++;

			HookInstrumentation();
		}
		bIsResettingShadow = false;
		bIsResettingShadowThread = false;
	}

	static TUniquePtr<FArchive> RaceDetectorLog;

	INSTRUMENTATION_FUNCTION_ATTRIBUTES void HandleReports()
	{
		TArray<FString> LocalReports;
		{
			TWriteScopeLock Lock(RaceReportsLock);
			LocalReports = MoveTemp(RaceReports);
		}

		if (LocalReports.Num())
		{
			if (RaceDetectorLog == nullptr)
			{
				const FString LogDir = GIsBuildMachine ? FPaths::Combine(*FPaths::EngineDir(), TEXT("Programs"), TEXT("AutomationTool"), TEXT("Saved"), TEXT("Logs")) : FPaths::ProjectLogDir();
				const FString StateLogOutputFilename = FPaths::Combine(LogDir, TEXT("Sanitizer"),
					FString::Printf(TEXT("RaceDetector-%08x-%s.log"), FPlatformProcess::GetCurrentProcessId(), *FDateTime::Now().ToIso8601().Replace(TEXT(":"), TEXT("."))));
				RaceDetectorLog = TUniquePtr<FArchive>(IFileManager::Get().CreateFileWriter(*StateLogOutputFilename, FILEWRITE_AllowRead));
			}

			for (FString& Report : LocalReports)
			{
				RaceDetectorLog->Serialize(TCHAR_TO_ANSI(*Report), Report.Len());
			}

			RaceDetectorLog->Flush();
		}
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES void CheckGracePeriod()
	{
		if (GlobalEpoch == GracePeriodEpoch && GracePeriodTimeout.IsExpired())
		{
			ToggleRaceDetection(false);
		}
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES void SanitizerThreadRun(volatile bool& bContinue)
	{
		FInstrumentationScope Scope;

		bool bWasDebuggerPresent = false;
		int32 MemoryUsageStatIterations = 0;
		while (bContinue)
		{
			Platform::SleepMS(1000);

			bIsDebuggerPresent = Platform::IsDebuggerPresent();
			if (bIsDebuggerPresent && !bWasDebuggerPresent)
			{
				Platform::HideFirstChanceExceptionInVisualStudio();
			}
			bWasDebuggerPresent = bIsDebuggerPresent;

			HandleReports();

			CheckGracePeriod();

			if (RaceDetectorActive)
			{
				if (RaceDetectorTimeout.IsExpired())
				{
					ToggleRaceDetection(false);
					continue;
				}

				double ShadowMemoryUsage = double(Platform::GetShadowMemoryUsage()) / (1024 *1024 *1024);
				double SyncObjectMemoryUsage = double(GetSyncObjectMemoryUsage()) / (1024 * 1024 * 1024);
				double HistoryChunkMemoryUsage = double(GetHistoryChunkMemoryUsage()) / (1024 * 1024 * 1024);

				if (++MemoryUsageStatIterations > 10)
				{
					FPlatformMisc::LowLevelOutputDebugStringf(
						TEXT("Sanitizer Memory Usage (Shadow : %.04f GB, SyncObjects : %.04f GB, History : %.04f GB)\n"),
						ShadowMemoryUsage,
						SyncObjectMemoryUsage,
						HistoryChunkMemoryUsage
					);
					MemoryUsageStatIterations = 0;
				}

				if (GRaceDetectorMaxMemoryUsage && (SyncObjectMemoryUsage + ShadowMemoryUsage) > GRaceDetectorMaxMemoryUsage)
				{
					ResetShadow();
				}
			}
		}

		HandleReports();
	}
	
	namespace Platform { INSTRUMENTATION_FUNCTION_ATTRIBUTES extern void ThrottleMemoryAccessesDuringShadowReset(); }

	/**
	 * FMalloc proxy removes instrumentation for allocators 
	 * and free the shadow range to avoid detecting races in freed
	 * memory.
	 */
	class AUTORTFM_DISABLE FMallocInstrumentation : public FMalloc
	{
	private:
		/** Malloc we're based on, aka using under the hood */
		FMalloc* InnerMalloc;

	public:
		SAFE_OPERATOR_NEW_DELETE();

		INSTRUMENTATION_FUNCTION_ATTRIBUTES explicit FMallocInstrumentation(FMalloc* InMalloc)
			: InnerMalloc(InMalloc)
		{
			checkf(InnerMalloc, TEXT("FMallocInstrumentation is used without a valid malloc!"));
		}

		INSTRUMENTATION_FUNCTION_ATTRIBUTES virtual void InitializeStatsMetadata() override
		{
			FInstrumentationScope Scope;
			InnerMalloc->InitializeStatsMetadata();
		}

		INSTRUMENTATION_FUNCTION_ATTRIBUTES virtual void* Malloc(SIZE_T Size, uint32 Alignment) override
		{
			FInstrumentationScope Scope;

			// Reduce the pressure on the memory allocator while we're resetting shadow as it can starve
			// the reset thread completely
			Platform::ThrottleMemoryAccessesDuringShadowReset();

			return InnerMalloc->Malloc(Size, Alignment);
		}

		INSTRUMENTATION_FUNCTION_ATTRIBUTES virtual void* Realloc(void* InPtr, SIZE_T NewSize, uint32 Alignment) override
		{
			FInstrumentationScope Scope;
			if (!RaceDetectorActive)
			{
				return InnerMalloc->Realloc(InPtr, NewSize, Alignment);
			}

			// Reduce the pressure on the memory allocator while we're resetting shadow as it can starve
			// the reset thread completely
			Platform::ThrottleMemoryAccessesDuringShadowReset();

			// We have to always allocate new blocs in order to invalidate the old memory range before it can be reused.
			void* NewPtr = nullptr;
			if (NewSize)
			{
				NewPtr = InnerMalloc->Malloc(NewSize, Alignment);
			}
			
			if (InPtr)
			{
				SIZE_T OldSize = 0;
				InnerMalloc->GetAllocationSize(InPtr, OldSize);

				if (OldSize)
				{
					if (NewPtr)
					{
						FMemory::Memcpy(NewPtr, InPtr, FMath::Min(OldSize, NewSize));
					}
					FreeMemoryRange(InPtr, OldSize);
				}
				InnerMalloc->Free(InPtr);
			}

			return NewPtr;
		}

		INSTRUMENTATION_FUNCTION_ATTRIBUTES virtual void Free(void* Ptr) override
		{
			FInstrumentationScope Scope;

			// Reduce the pressure on the memory allocator while we're resetting shadow as it can starve
			// the reset thread completely
			Platform::ThrottleMemoryAccessesDuringShadowReset();

			SIZE_T Size = 0;
			InnerMalloc->GetAllocationSize(Ptr, Size);
			if (Ptr && Size)
			{
				FreeMemoryRange(Ptr, Size);
			}
			return InnerMalloc->Free(Ptr);
		}

		INSTRUMENTATION_FUNCTION_ATTRIBUTES virtual SIZE_T QuantizeSize(SIZE_T Count, uint32 Alignment) override
		{
			FInstrumentationScope Scope;
			return InnerMalloc->QuantizeSize(Count, Alignment);
		}

		INSTRUMENTATION_FUNCTION_ATTRIBUTES virtual void UpdateStats() override
		{
			FInstrumentationScope Scope;
			InnerMalloc->UpdateStats();
		}

		INSTRUMENTATION_FUNCTION_ATTRIBUTES virtual void GetAllocatorStats(FGenericMemoryStats& out_Stats) override
		{
			FInstrumentationScope Scope;
			InnerMalloc->GetAllocatorStats(out_Stats);
		}

		INSTRUMENTATION_FUNCTION_ATTRIBUTES virtual void DumpAllocatorStats(class FOutputDevice& Ar) override
		{
			FInstrumentationScope Scope;
			InnerMalloc->DumpAllocatorStats(Ar);
		}

		INSTRUMENTATION_FUNCTION_ATTRIBUTES virtual bool IsInternallyThreadSafe() const override
		{
			FInstrumentationScope Scope;
			return InnerMalloc->IsInternallyThreadSafe();
		}

		INSTRUMENTATION_FUNCTION_ATTRIBUTES virtual bool ValidateHeap() override
		{
			FInstrumentationScope Scope;
			return InnerMalloc->ValidateHeap();
		}

#if UE_ALLOW_EXEC_COMMANDS
		INSTRUMENTATION_FUNCTION_ATTRIBUTES virtual bool Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override
		{
			FInstrumentationScope Scope;
			return InnerMalloc->Exec(InWorld, Cmd, Ar);
		}
#endif // UE_ALLOW_EXEC_COMMANDS

		INSTRUMENTATION_FUNCTION_ATTRIBUTES virtual bool GetAllocationSize(void* Ptr, SIZE_T& SizeOut) override
		{
			FInstrumentationScope Scope;
			return InnerMalloc->GetAllocationSize(Ptr, SizeOut);
		}

		INSTRUMENTATION_FUNCTION_ATTRIBUTES virtual const TCHAR* GetDescriptiveName() override
		{
			FInstrumentationScope Scope;
			return InnerMalloc->GetDescriptiveName();
		}

		INSTRUMENTATION_FUNCTION_ATTRIBUTES virtual void Trim(bool bTrimThreadCaches) override
		{
			FInstrumentationScope Scope;
			InnerMalloc->Trim(bTrimThreadCaches);
		}

		INSTRUMENTATION_FUNCTION_ATTRIBUTES virtual void SetupTLSCachesOnCurrentThread() override
		{
			FInstrumentationScope Scope;
			InnerMalloc->SetupTLSCachesOnCurrentThread();
		}

		INSTRUMENTATION_FUNCTION_ATTRIBUTES virtual void MarkTLSCachesAsUsedOnCurrentThread() override
		{
			FInstrumentationScope Scope;
			InnerMalloc->MarkTLSCachesAsUsedOnCurrentThread();
		}

		INSTRUMENTATION_FUNCTION_ATTRIBUTES virtual void MarkTLSCachesAsUnusedOnCurrentThread() override
		{
			FInstrumentationScope Scope;
			InnerMalloc->MarkTLSCachesAsUnusedOnCurrentThread();
		}

		INSTRUMENTATION_FUNCTION_ATTRIBUTES virtual void ClearAndDisableTLSCachesOnCurrentThread() override
		{
			FInstrumentationScope Scope;
			InnerMalloc->ClearAndDisableTLSCachesOnCurrentThread();
		}

		INSTRUMENTATION_FUNCTION_ATTRIBUTES virtual void OnMallocInitialized() override
		{
			FInstrumentationScope Scope;
			InnerMalloc->OnMallocInitialized();
		}

		INSTRUMENTATION_FUNCTION_ATTRIBUTES virtual void OnPreFork() override
		{
			FInstrumentationScope Scope;
			InnerMalloc->OnPreFork();
		}

		INSTRUMENTATION_FUNCTION_ATTRIBUTES virtual void OnPostFork() override
		{
			FInstrumentationScope Scope;
			InnerMalloc->OnPostFork();
		}

		// FMalloc interface end
	};

	INSTRUMENTATION_FUNCTION_ATTRIBUTES FInstrumentationScope::FInstrumentationScope()
	{
		// This is important since we're never unregistering the malloc instrumentation so we need to avoid
		// handling context once the runtime is shut down otherwise we can end up with use-after-free on the
		// TLS during application exit.
		if (bRuntimeInitialized)
		{
			FContext* Context = GetThreadContext();
			if (FContext::IsValid(Context))
			{
				Context->WinInstrumentationDepth++;
				bNeedDecrement = true;
			}
		}
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES void InitMemoryAllocator()
	{
		while (true)
		{
			FMalloc* LocalGMalloc = UE::Private::GMalloc;
			FMalloc* Proxy = new FMallocInstrumentation(LocalGMalloc);
			if (FPlatformAtomics::InterlockedCompareExchangePointer((void**)&UE::Private::GMalloc, Proxy, LocalGMalloc) == LocalGMalloc)
			{
				return;
			}
			delete Proxy;
		}
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES void PopulateHotPatchFunctions();
	INSTRUMENTATION_FUNCTION_ATTRIBUTES void PrepareHotPatchFunctions();
	INSTRUMENTATION_FUNCTION_ATTRIBUTES void HookAlwaysOnInstrumentation();
	INSTRUMENTATION_FUNCTION_ATTRIBUTES bool Initialize()
	{
		if (bRuntimeInitialized)
		{
			return true;
		}

		PopulateHotPatchFunctions();

		PrepareHotPatchFunctions();

		if (!Platform::InitializePlatform())
		{
			return false;
		}

		Platform::InitShadowMemory();

		// Reserve virtual address space for the sync object linear allocator.
		// 64GB reserved (only committed on demand in 2MB chunks).
		Platform::InitSyncObjectAllocator(64ull * 1024 * 1024 * 1024, 2 * 1024 * 1024);

		ShadowMemoryBase = Platform::GetShadowMemoryBase();
		ShadowMemorySize = Platform::GetShadowMemorySize();
		ShadowMemoryEnd = ShadowMemoryBase + ShadowMemorySize;
		ShadowClockBase = Platform::GetShadowClockBase();

		InitMemoryAllocator();
		
		// Set as initialized now otherwise we can end up with messed up callstack state
		// since we start using incrementation scope inside HookAlwaysOnInstrumentation.
		bRuntimeInitialized = true;

		HookAlwaysOnInstrumentation();
		
		const TCHAR* CommandLine = Platform::GetCommandLine();

		FString RaceDetectorLogFile;
		if (FParse::Value(CommandLine, TEXT("-racedetectorlog="), RaceDetectorLogFile))
		{
			RaceDetectorLog = TUniquePtr<FArchive>(IFileManager::Get().CreateFileWriter(*RaceDetectorLogFile, FILEWRITE_AllowRead));
		}

		if (FParse::Param(CommandLine, TEXT("-racedetector")))
		{
			ToggleRaceDetection(true);
		}

		return true;
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES FContext& EnsureCurrentContext()
	{
		RACEDETECTOR_CHECK(ContextTLSValue != 0 && ContextTLSIndex != UINT32_MAX);
		FContext* Context = GetThreadContext();
		if (Context == nullptr)
		{
			Context = new FContext(Platform::GetCurrentThreadId());

			// Refcount owned by the thread itself.
			Context->AddRef();
			Platform::SetTlsValue(ContextTLSIndex, Context);
			if (GetTlsValueFast(ContextTLSValue) != Context)
			{
				UE_LOGF(LogRaceDetector, Fatal, "GetTlsValueFast implementation is invalid");
			}
		}

		return *Context;
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES void ReleaseCurrentContext()
	{
		if (FContext* Context = GetThreadContext(); FContext::IsValid(Context))
		{
			// Mark ContextTLS as invalid to prevent any further usage / detection for this thread
			Platform::SetTlsValue(ContextTLSIndex, (void*)-1);

			// We don't care about the depth we're currently in when releasing the current context 
			// since this only happens during shutdown and thread cleanup.
			Context->InstrumentationDepth = 0;

			ReleaseContext(Context);

			// Release the refcount owned by the thread.
			Context->Release();
		}
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES void CleanupHotPatchFunctions();
	INSTRUMENTATION_FUNCTION_ATTRIBUTES void ReleaseCurrentContext();
	INSTRUMENTATION_FUNCTION_ATTRIBUTES void UnhookAlwaysOnInstrumentation();
	INSTRUMENTATION_FUNCTION_ATTRIBUTES bool Shutdown()
	{
		if (!bRuntimeInitialized)
		{
			return true;
		}
		
		// Needed to avoid race with ToggleRaceDetection
		UE::TUniqueLock ScopeLock(Mutex);

		UnhookInstrumentation();

		ResetShadow();

		Platform::ShutdownSyncObjectAllocator();

		UnhookAlwaysOnInstrumentation();

		CleanupHotPatchFunctions();

		if (!Platform::CleanupPlatform())
		{
			return false;
		}

		if (ContextTLSIndex != UINT32_MAX)
		{
			ReleaseCurrentContext();

			// Because we unhooked the detoured function doesn't mean there isn't a call still inside one.
			// We have no choice but to leak the context tls to avoid crashes if that ever happens.
		}

		bRuntimeInitialized = false;
		return true;
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES void ToggleFilterDuplicateRaces(bool bEnable)
	{
		GRaceDetectorFilterDuplicates = bEnable;
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES void ToggleFilterOtherThreads(bool bEnable)
	{
		GRaceDetectorFilterOtherThreads = bEnable;
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES bool IsActive()
	{
		return RaceDetectorActive != 0;
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES void ToggleRaceDetectionInternal(bool bEnable, float GracePeriodSeconds, UE::FTimeout Timeout = UE::FTimeout::Never())
	{
		int32 Result = 0;

		// Just make sure we're not initializing the stack from an access that comes from this function
		{
			// Need to take the mutex during reset shadow since we're unhooking and rehooking instrumentation, which is not thread-safe.
			UE::TUniqueLock ScopeLock(Mutex);

			if (!bRuntimeInitialized)
			{
				return;
			}

			FContext& Context = EnsureCurrentContext();

			FInstrumentationScope InstrumentationScope;
			if (bEnable)
			{
				Context.bAlwaysReport = true;

				RaceDetectorTimeout = Timeout;
				if (RaceDetectorActive++ == 0)
				{
					GracePeriodTimeout = UE::FTimeout::Never();

					GlobalEpoch++;

					Result = 1;
					HookInstrumentation();
				}
			}
			else
			{
				Context.bAlwaysReport = false;

				if (GracePeriodSeconds > 0.0f)
				{
					// There is already a grace period active, decrement the count to avoid accumulating
					if (!GracePeriodTimeout.WillNeverExpire())
					{
						if (RaceDetectorActive > 1)
						{
							--RaceDetectorActive;
						}
					}
					GracePeriodTimeout = UE::FTimeout(GracePeriodSeconds);
					GracePeriodEpoch = GlobalEpoch;
				}
				else
				{
					if (RaceDetectorActive > 0 && --RaceDetectorActive == 0)
					{
						RaceDetectorTimeout = UE::FTimeout::Never();
						GracePeriodTimeout = UE::FTimeout::Never();

						Result = 2;
						UnhookInstrumentation();
						ResetShadow();
					}
				}
			}
		}

		// This needs to be properly instrumented because it does synchronize with another thread so if the
		// instrumentation doesn't see the atomics being used from this thread, a future race will be
		// detected when the stack memory used as synchronization during logging starts being reused.
		if (Result)
		{
			UE_LOGF(LogRaceDetector, Log, "Race detector has been toggled %ls", Result == 1 ? TEXT("on") : TEXT("off"))
		}
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES void ToggleRaceDetectionUntil(UE::FTimeout Timeout)
	{
		ToggleRaceDetectionInternal(true, .0f, Timeout);
	}

	INSTRUMENTATION_ENTRYPOINT_ATTRIBUTES void ToggleRaceDetection(bool bEnable, float GracePeriodSeconds)
	{
		ToggleRaceDetectionInternal(bEnable, GracePeriodSeconds);
	}

	INSTRUMENTATION_ENTRYPOINT_ATTRIBUTES void ToggleThreadDetailedLog(bool bEnabled)
	{
		FContext& Context = EnsureCurrentContext();
		if (bEnabled)
		{
			Context.DetailedLogDepth++;
		}
		else if (Context.DetailedLogDepth > 0)
		{
			Context.DetailedLogDepth--;
		}
	}

	INSTRUMENTATION_ENTRYPOINT_ATTRIBUTES void ToggleGlobalDetailedLog(bool bEnabled)
	{
		bDetailedLogGlobal = bEnabled;
	}

	INSTRUMENTATION_ENTRYPOINT_ATTRIBUTES void ToggleFilterDetailedLogOnAddress(void* Address)
	{
		DetailedLogAddress = Address;
	}


} // UE::Sanitizer::RaceDetector

using namespace UE::Sanitizer;
using namespace UE::Sanitizer::RaceDetector;

// We don't need to test for bRaceDetectorActive or bIsResettingShadow here
// since we unhook the instrumentation instead.
#define BEGIN_HANDLE_INSTRUMENTATION(FuncDoIfNotInstrumenting) \
	FContext* ContextPtr = (FContext*)GetTlsValueFast(ContextTLSValue); \
	if (!FContext::IsValid(ContextPtr) || (ContextPtr->WinInstrumentationDepth | ContextPtr->InstrumentationDepth) != 0) \
	{ \
		FuncDoIfNotInstrumenting; \
	} \
	FContext& Context = *ContextPtr; \
	Context.InstrumentationDepth++; \

#define FINISH_HANDLE_INSTRUMENTATION() \
	Context.InstrumentationDepth--;

#define INSTRUMENTATION_CORE_FUNCTION_PREFIX INSTRUMENTATION_FUNCTION_ATTRIBUTES CORE_API

extern "C" {

	INSTRUMENTATION_CORE_FUNCTION_PREFIX void __RaceDetector__AnnotateHappensBefore(const char* f, int l, void* addr)
	{
		BEGIN_HANDLE_INSTRUMENTATION(return);

		AddCallstackFrame(Context, _ReturnAddress());
		FSyncObjectRef Atomic = GetSyncObject(Context, addr);
		Atomic->SyncRelease(Context, []() {}, _ReturnAddress(), addr, TEXT("AnnotateHappensBefore"));
		Context.IncrementClock();
		RemoveCallstackFrame(Context);

		FINISH_HANDLE_INSTRUMENTATION();
	}

	INSTRUMENTATION_CORE_FUNCTION_PREFIX void __RaceDetector__AnnotateHappensAfter(const char* f, int l, void* addr)
	{
		BEGIN_HANDLE_INSTRUMENTATION(return);

		AddCallstackFrame(Context, _ReturnAddress());
		FSyncObjectRef Atomic = GetSyncObject(Context, addr);
		Atomic->SyncAcquire(Context, []() {}, _ReturnAddress(), addr, TEXT("AnnotateHappensAfter"));
		Context.IncrementClock();
		RemoveCallstackFrame(Context);

		FINISH_HANDLE_INSTRUMENTATION();
	}

	INSTRUMENTATION_CORE_FUNCTION_PREFIX void __RaceDetector__Instrument_FuncEntry(void* ReturnAddress)
	{
		BEGIN_HANDLE_INSTRUMENTATION(return);

		AddCallstackFrame(Context, ReturnAddress);

		FINISH_HANDLE_INSTRUMENTATION();
	}

	INSTRUMENTATION_CORE_FUNCTION_PREFIX void __RaceDetector__Instrument_FuncExit()
	{
		BEGIN_HANDLE_INSTRUMENTATION(return);

		RemoveCallstackFrame(Context);

		FINISH_HANDLE_INSTRUMENTATION();
	}

	INSTRUMENTATION_CORE_FUNCTION_PREFIX __attribute__((hot)) void __RaceDetector__Instrument_Store(uint64 Address, uint32 Size)
	{
		BEGIN_HANDLE_INSTRUMENTATION(return);

		bool bHasAddedCallstackFrame = false;
		InstrumentAccessPreamble(Context, Address, Size, EMemoryAccessType::ACCESS_TYPE_WRITE, FAtomicMemoryOrder::MEMORY_ORDER_RELAXED, _ReturnAddress(), TEXT("Store"), bHasAddedCallstackFrame);
		if (bHasAddedCallstackFrame)
		{
			RemoveCallstackFrame(Context);
		}

		FINISH_HANDLE_INSTRUMENTATION();
	}

	INSTRUMENTATION_CORE_FUNCTION_PREFIX __attribute__((hot)) void __RaceDetector__Instrument_Load(uint64 Address, uint32 Size)
	{
		BEGIN_HANDLE_INSTRUMENTATION(return);

		bool bHasAddedCallstackFrame = false;
		InstrumentAccessPreamble(Context, Address, Size, EMemoryAccessType::ACCESS_TYPE_READ, FAtomicMemoryOrder::MEMORY_ORDER_RELAXED, _ReturnAddress(), TEXT("Load"), bHasAddedCallstackFrame);
		if (bHasAddedCallstackFrame)
		{
			RemoveCallstackFrame(Context);
		}

		FINISH_HANDLE_INSTRUMENTATION();
	}

	INSTRUMENTATION_CORE_FUNCTION_PREFIX void __RaceDetector__Instrument_VPtr_Store(void** Address, void* Value)
	{
		BEGIN_HANDLE_INSTRUMENTATION(return);

		// For virtual table pointers, there is no race if the store is simply rewriting the same value as this will generally happen
		// when entering a destructor of a base class that wasn't subclassed.
		// See FRaceDetectorVirtualPointerBenignTest in RaceDetectorTests.cpp for an example.
		if (*Address != Value)
		{
			bool bHasAddedCallstackFrame = false;
			InstrumentAccessPreamble(Context, (UPTRINT)Address, sizeof(void*), EMemoryAccessType::ACCESS_TYPE_WRITE | EMemoryAccessType::ACCESS_TYPE_VPTR, FAtomicMemoryOrder::MEMORY_ORDER_RELAXED, _ReturnAddress(), TEXT("VPtr Store"), bHasAddedCallstackFrame);
			if (bHasAddedCallstackFrame)
			{
				RemoveCallstackFrame(Context);
			}
		}

		FINISH_HANDLE_INSTRUMENTATION();
	}

	INSTRUMENTATION_CORE_FUNCTION_PREFIX void __RaceDetector__Instrument_VPtr_Load(void** Address)
	{
		BEGIN_HANDLE_INSTRUMENTATION(return);

		bool bHasAddedCallstackFrame = false;
		InstrumentAccessPreamble(Context, (UPTRINT)Address, sizeof(void*), EMemoryAccessType::ACCESS_TYPE_READ | EMemoryAccessType::ACCESS_TYPE_VPTR, FAtomicMemoryOrder::MEMORY_ORDER_RELAXED, _ReturnAddress(), TEXT("VPtr Load"), bHasAddedCallstackFrame);
		if (bHasAddedCallstackFrame)
		{
			RemoveCallstackFrame(Context);
		}

		FINISH_HANDLE_INSTRUMENTATION();
	}

	INSTRUMENTATION_CORE_FUNCTION_PREFIX void __RaceDetector__Instrument_StoreRange(uint64 Address, uint32 Size)
	{
		if (Size == 0)
		{
			return;
		}

		BEGIN_HANDLE_INSTRUMENTATION(return);
		
		bool bHasAddedCallstackFrame = false;
		InstrumentAccessPreamble(Context, Address, Size, EMemoryAccessType::ACCESS_TYPE_WRITE, FAtomicMemoryOrder::MEMORY_ORDER_RELAXED, _ReturnAddress(), TEXT("StoreRange"), bHasAddedCallstackFrame);
		if (bHasAddedCallstackFrame)
		{
			RemoveCallstackFrame(Context);
		}

		FINISH_HANDLE_INSTRUMENTATION();
	}

	INSTRUMENTATION_CORE_FUNCTION_PREFIX void __RaceDetector__Instrument_LoadRange(uint64 Address, uint32 Size)
	{
		if (Size == 0)
		{
			return;
		}

		BEGIN_HANDLE_INSTRUMENTATION(return);

		bool bHasAddedCallstackFrame = false;
		InstrumentAccessPreamble(Context, Address, Size, EMemoryAccessType::ACCESS_TYPE_READ, FAtomicMemoryOrder::MEMORY_ORDER_RELAXED, _ReturnAddress(), TEXT("LoadRange"), bHasAddedCallstackFrame);
		if (bHasAddedCallstackFrame)
		{
			RemoveCallstackFrame(Context);
		}

		FINISH_HANDLE_INSTRUMENTATION();
	}
}

template <typename AtomicType>
INSTRUMENTATION_FUNCTION_ATTRIBUTES FORCEINLINE AtomicType InstrumentAtomicLoad(AtomicType* Atomic, FAtomicMemoryOrder Order)
{
	AtomicType Ret;
	auto AtomicOp = [Atomic, &Ret]() INSTRUMENTATION_FUNCTION_ATTRIBUTES {
		Ret = UE::Core::Private::Atomic::Load(Atomic);
		};

	BEGIN_HANDLE_INSTRUMENTATION(AtomicOp(); return Ret);
	bool bHasAddedCallstackFrame = false;
	InstrumentAtomicAccess(Context, Atomic, sizeof(AtomicType), EMemoryAccessType::ACCESS_TYPE_ATOMIC_READ, Order, AtomicOp, _ReturnAddress(), TEXT("AtomicLoad"), bHasAddedCallstackFrame);
	if (bHasAddedCallstackFrame)
	{
		RemoveCallstackFrame(Context);
	}
	FINISH_HANDLE_INSTRUMENTATION()
	return Ret;
}

template <typename AtomicType>
INSTRUMENTATION_FUNCTION_ATTRIBUTES FORCEINLINE void InstrumentAtomicStore(AtomicType* Atomic, AtomicType Val, FAtomicMemoryOrder Order)
{
	auto AtomicOp = [Atomic, Val]() INSTRUMENTATION_FUNCTION_ATTRIBUTES {
		UE::Core::Private::Atomic::Store(Atomic, Val);
		};

	BEGIN_HANDLE_INSTRUMENTATION(AtomicOp(); return);
	bool bHasAddedCallstackFrame = false;
	InstrumentAtomicAccess(Context, Atomic, sizeof(AtomicType), EMemoryAccessType::ACCESS_TYPE_ATOMIC_WRITE, Order, AtomicOp, _ReturnAddress(), TEXT("AtomicStore"), bHasAddedCallstackFrame);
	if (bHasAddedCallstackFrame)
	{
		RemoveCallstackFrame(Context);
	}
	FINISH_HANDLE_INSTRUMENTATION();
}

template <typename AtomicType>
INSTRUMENTATION_FUNCTION_ATTRIBUTES FORCEINLINE AtomicType InstrumentAtomicExchange(AtomicType* Atomic, AtomicType Val, FAtomicMemoryOrder Order)
{
	AtomicType Ret;
	auto AtomicOp = [Atomic, Val, &Ret]() INSTRUMENTATION_FUNCTION_ATTRIBUTES {
		Ret = UE::Core::Private::Atomic::Exchange(Atomic, Val);
		};

	BEGIN_HANDLE_INSTRUMENTATION(AtomicOp(); return Ret);
	bool bHasAddedCallstackFrame = false;
	InstrumentAtomicAccess(Context, Atomic, sizeof(AtomicType), EMemoryAccessType::ACCESS_TYPE_ATOMIC_READ_WRITE, Order, AtomicOp, _ReturnAddress(), TEXT("AtomicExchange"), bHasAddedCallstackFrame);
	if (bHasAddedCallstackFrame)
	{
		RemoveCallstackFrame(Context);
	}
	FINISH_HANDLE_INSTRUMENTATION();
	return Ret;
}

template <typename AtomicType>
INSTRUMENTATION_FUNCTION_ATTRIBUTES FORCEINLINE AtomicType InstrumentAtomicCompareExchange(AtomicType* Atomic, AtomicType* Expected, AtomicType Val, FAtomicMemoryOrder SuccessOrder, FAtomicMemoryOrder FailureOrder)
{
	AtomicType Ret;
	auto AtomicOp = [Atomic, Expected, Val, &Ret]() INSTRUMENTATION_FUNCTION_ATTRIBUTES -> bool {
		Ret = UE::Core::Private::Atomic::CompareExchange(Atomic, *Expected, Val);
		if (Ret != *Expected)
		{
			*Expected = Ret;
			return false;
		}
		return true;
	};

	BEGIN_HANDLE_INSTRUMENTATION(AtomicOp(); return Ret);
	bool bHasAddedCallstackFrame = false;
	InstrumentAtomicAccess(Context, Atomic, sizeof(AtomicType), EMemoryAccessType::ACCESS_TYPE_ATOMIC_READ_WRITE, SuccessOrder, FailureOrder, AtomicOp, _ReturnAddress(), TEXT("CompareExchange"), bHasAddedCallstackFrame);
	if (bHasAddedCallstackFrame)
	{
		RemoveCallstackFrame(Context);
	}
	FINISH_HANDLE_INSTRUMENTATION();
	return Ret;
}

template <typename AtomicType>
INSTRUMENTATION_FUNCTION_ATTRIBUTES FORCEINLINE AtomicType InstrumentAtomicFetchAdd(AtomicType* Atomic, AtomicType Val, FAtomicMemoryOrder Order)
{
	AtomicType Ret;
	auto AtomicOp = [Atomic, Val, &Ret]() INSTRUMENTATION_FUNCTION_ATTRIBUTES {
		Ret = UE::Core::Private::Atomic::AddExchange(Atomic, Val);
		};

	BEGIN_HANDLE_INSTRUMENTATION(AtomicOp(); return Ret);
	bool bHasAddedCallstackFrame = false;
	InstrumentAtomicAccess(Context, Atomic, sizeof(AtomicType), EMemoryAccessType::ACCESS_TYPE_ATOMIC_READ_WRITE, Order, AtomicOp, _ReturnAddress(), TEXT("FetchAdd"), bHasAddedCallstackFrame);
	if (bHasAddedCallstackFrame)
	{
		RemoveCallstackFrame(Context);
	}
	FINISH_HANDLE_INSTRUMENTATION();
	return Ret;
}

template <typename AtomicType>
INSTRUMENTATION_FUNCTION_ATTRIBUTES FORCEINLINE AtomicType InstrumentAtomicFetchSub(AtomicType* Atomic, AtomicType Val, FAtomicMemoryOrder Order)
{
	AtomicType Ret;
	auto AtomicOp = [Atomic, Val, &Ret]() INSTRUMENTATION_FUNCTION_ATTRIBUTES {
		Ret = UE::Core::Private::Atomic::SubExchange(Atomic, Val);
		};

	BEGIN_HANDLE_INSTRUMENTATION(AtomicOp(); return Ret);
	bool bHasAddedCallstackFrame = false;
	InstrumentAtomicAccess(Context, Atomic, sizeof(AtomicType), EMemoryAccessType::ACCESS_TYPE_ATOMIC_READ_WRITE, Order, AtomicOp, _ReturnAddress(), TEXT("FetchSub"), bHasAddedCallstackFrame);
	if (bHasAddedCallstackFrame)
	{
		RemoveCallstackFrame(Context);
	}
	FINISH_HANDLE_INSTRUMENTATION();
	return Ret;
}

template <typename AtomicType>
INSTRUMENTATION_FUNCTION_ATTRIBUTES FORCEINLINE AtomicType InstrumentAtomicFetchOr(AtomicType* Atomic, AtomicType Val, FAtomicMemoryOrder Order)
{
	AtomicType Ret;
	auto AtomicOp = [Atomic, Val, &Ret]() INSTRUMENTATION_FUNCTION_ATTRIBUTES {
		Ret = UE::Core::Private::Atomic::OrExchange(Atomic, Val);
		};

	BEGIN_HANDLE_INSTRUMENTATION(AtomicOp(); return Ret);
	bool bHasAddedCallstackFrame = false;
	InstrumentAtomicAccess(Context, Atomic, sizeof(AtomicType), EMemoryAccessType::ACCESS_TYPE_ATOMIC_READ_WRITE, Order, AtomicOp, _ReturnAddress(), TEXT("FetchOr"), bHasAddedCallstackFrame);
	if (bHasAddedCallstackFrame)
	{
		RemoveCallstackFrame(Context);
	}
	FINISH_HANDLE_INSTRUMENTATION();
	return Ret;
}

template <typename AtomicType>
INSTRUMENTATION_FUNCTION_ATTRIBUTES FORCEINLINE AtomicType InstrumentAtomicFetchXor(AtomicType* Atomic, AtomicType Val, FAtomicMemoryOrder Order)
{
	AtomicType Ret;
	auto AtomicOp = [Atomic, Val, &Ret]() INSTRUMENTATION_FUNCTION_ATTRIBUTES {
		Ret = UE::Core::Private::Atomic::XorExchange(Atomic, Val);
	};
	
	BEGIN_HANDLE_INSTRUMENTATION(AtomicOp(); return Ret);
	bool bHasAddedCallstackFrame = false;
	InstrumentAtomicAccess(Context, Atomic, sizeof(AtomicType), EMemoryAccessType::ACCESS_TYPE_ATOMIC_READ_WRITE, Order, AtomicOp, _ReturnAddress(), TEXT("FetchXor"), bHasAddedCallstackFrame);
	if (bHasAddedCallstackFrame)
	{
		RemoveCallstackFrame(Context);
	}
	FINISH_HANDLE_INSTRUMENTATION();
	return Ret;
}

template <typename AtomicType>
INSTRUMENTATION_FUNCTION_ATTRIBUTES FORCEINLINE AtomicType InstrumentAtomicFetchAnd(AtomicType* Atomic, AtomicType Val, FAtomicMemoryOrder Order)
{
	AtomicType Ret;
	auto AtomicOp = [Atomic, Val, &Ret]() INSTRUMENTATION_FUNCTION_ATTRIBUTES {
		Ret = UE::Core::Private::Atomic::AndExchange(Atomic, Val);
		};
	
	BEGIN_HANDLE_INSTRUMENTATION(AtomicOp(); return Ret);
	bool bHasAddedCallstackFrame = false;
	InstrumentAtomicAccess(Context, Atomic, sizeof(AtomicType), EMemoryAccessType::ACCESS_TYPE_ATOMIC_READ_WRITE, Order, AtomicOp, _ReturnAddress(), TEXT("FetchAnd"), bHasAddedCallstackFrame);
	if (bHasAddedCallstackFrame)
	{
		RemoveCallstackFrame(Context);
	}
	FINISH_HANDLE_INSTRUMENTATION();
	return Ret;
}

#define INSTRUMENT_LOAD_FUNC(Type) \
INSTRUMENTATION_CORE_FUNCTION_PREFIX Type __RaceDetector__Instrument_AtomicLoad_##Type(Type* Atomic, FAtomicMemoryOrder MemoryOrder) \
{ \
	return InstrumentAtomicLoad(Atomic, MemoryOrder); \
}

#define INSTRUMENT_STORE_FUNC(Type) \
INSTRUMENTATION_CORE_FUNCTION_PREFIX void __RaceDetector__Instrument_AtomicStore_##Type(Type* Atomic, Type Val, FAtomicMemoryOrder MemoryOrder) \
{ \
	return InstrumentAtomicStore(Atomic, Val, MemoryOrder); \
}

#define INSTRUMENT_EXCHANGE_FUNC(Type) \
INSTRUMENTATION_CORE_FUNCTION_PREFIX Type __RaceDetector__Instrument_AtomicExchange_##Type(Type* Atomic, Type Val, FAtomicMemoryOrder MemoryOrder) \
{ \
    return InstrumentAtomicExchange(Atomic, Val, MemoryOrder); \
}

#define INSTRUMENT_COMPARE_EXCHANGE_FUNC(Type) \
INSTRUMENTATION_CORE_FUNCTION_PREFIX Type __RaceDetector__Instrument_AtomicCompareExchange_##Type(Type* Atomic, Type* Expected, Type Val, FAtomicMemoryOrder SuccessMemoryOrder, FAtomicMemoryOrder FailureMemoryOrder) \
{ \
    return InstrumentAtomicCompareExchange(Atomic, Expected, Val, SuccessMemoryOrder, FailureMemoryOrder); \
} 

#define INSTRUMENT_RMW_FUNC(Func, Type) \
INSTRUMENTATION_CORE_FUNCTION_PREFIX Type __RaceDetector__Instrument_Atomic##Func##_##Type(Type* Atomic, Type Val, FAtomicMemoryOrder MemoryOrder) \
{ \
	return InstrumentAtomic##Func(Atomic, Val, MemoryOrder); \
}

// Define the functions with C linkage, referenced by the compiler's instrumentation pass.
extern "C" {
	INSTRUMENT_LOAD_FUNC(int8)
	INSTRUMENT_LOAD_FUNC(int16)
	INSTRUMENT_LOAD_FUNC(int32)
	INSTRUMENT_LOAD_FUNC(int64)

	INSTRUMENT_STORE_FUNC(int8)
	INSTRUMENT_STORE_FUNC(int16)
	INSTRUMENT_STORE_FUNC(int32)
	INSTRUMENT_STORE_FUNC(int64)

	INSTRUMENT_EXCHANGE_FUNC(int8)
	INSTRUMENT_EXCHANGE_FUNC(int16)
	INSTRUMENT_EXCHANGE_FUNC(int32)
	INSTRUMENT_EXCHANGE_FUNC(int64)

	INSTRUMENT_COMPARE_EXCHANGE_FUNC(int8)
	INSTRUMENT_COMPARE_EXCHANGE_FUNC(int16)
	INSTRUMENT_COMPARE_EXCHANGE_FUNC(int32)
	INSTRUMENT_COMPARE_EXCHANGE_FUNC(int64)

	INSTRUMENT_RMW_FUNC(FetchAdd, int8)
	INSTRUMENT_RMW_FUNC(FetchAdd, int16)
	INSTRUMENT_RMW_FUNC(FetchAdd, int32)
	INSTRUMENT_RMW_FUNC(FetchAdd, int64)
	INSTRUMENT_RMW_FUNC(FetchSub, int8)
	INSTRUMENT_RMW_FUNC(FetchSub, int16)
	INSTRUMENT_RMW_FUNC(FetchSub, int32)
	INSTRUMENT_RMW_FUNC(FetchSub, int64)
	INSTRUMENT_RMW_FUNC(FetchOr, int8)
	INSTRUMENT_RMW_FUNC(FetchOr, int16)
	INSTRUMENT_RMW_FUNC(FetchOr, int32)
	INSTRUMENT_RMW_FUNC(FetchOr, int64)
	INSTRUMENT_RMW_FUNC(FetchXor, int8)
	INSTRUMENT_RMW_FUNC(FetchXor, int16)
	INSTRUMENT_RMW_FUNC(FetchXor, int32)
	INSTRUMENT_RMW_FUNC(FetchXor, int64)
	INSTRUMENT_RMW_FUNC(FetchAnd, int8)
	INSTRUMENT_RMW_FUNC(FetchAnd, int16)
	INSTRUMENT_RMW_FUNC(FetchAnd, int32)
	INSTRUMENT_RMW_FUNC(FetchAnd, int64)
}


// We need the pointers to the native instrument functions to hotpatch them.
extern "C" {
	void AnnotateHappensBefore(const char* f, int l, void* addr);
	void AnnotateHappensAfter(const char* f, int l, void* addr);
	void __Instrument_FuncEntry(void* ReturnAddress);
	void __Instrument_FuncExit();
	void __Instrument_StoreRange(uint64 Address, uint32 Size);
	void __Instrument_LoadRange(uint64 Address, uint32 Size);
	void __Instrument_Store(uint64 Address, uint32 Size);
	void __Instrument_Load(uint64 Address, uint32 Size);
	void __Instrument_VPtr_Store(void** Address, void* Value);
	void __Instrument_VPtr_Load(void** Address);
}

namespace UE::Sanitizer::RaceDetector {

// Those are never toggled off after boot.
TArray<TPair<void*, void*>>  AlwaysOnInstrumentationFunctions;
// We use our own hotpatching method to hook into these atomically.
TArray<TPair<void*, void*>>  HotpatchInstrumentationFunctions;
// We can place the hottest function that do nothing in a fast collection so they get patched by a RET when inactive.
// This saves more than 1m30s on a 8m15s map load!!
TArray<TPair<void*, void*>>  HotpatchInstrumentationFunctionsFast;

INSTRUMENTATION_FUNCTION_ATTRIBUTES void PopulateHotPatchFunctions()
{
	if (!HotpatchInstrumentationFunctions.IsEmpty())
	{
		return;
	}

	AlwaysOnInstrumentationFunctions.Emplace(__Thunk__Instrument_FuncEntry, __RaceDetector__Instrument_FuncEntry);
	AlwaysOnInstrumentationFunctions.Emplace(__Thunk__Instrument_FuncExit, __RaceDetector__Instrument_FuncExit);

	// Hook the instrumentation thunks needed for race detector, this is where the calls from all the modules end up.
	HotpatchInstrumentationFunctionsFast.Emplace(__Thunk__AnnotateHappensBefore, __RaceDetector__AnnotateHappensBefore);
	HotpatchInstrumentationFunctionsFast.Emplace(__Thunk__AnnotateHappensAfter, __RaceDetector__AnnotateHappensAfter);
	HotpatchInstrumentationFunctionsFast.Emplace(__Thunk__Instrument_StoreRange, __RaceDetector__Instrument_StoreRange);
	HotpatchInstrumentationFunctionsFast.Emplace(__Thunk__Instrument_LoadRange, __RaceDetector__Instrument_LoadRange);
	HotpatchInstrumentationFunctionsFast.Emplace(__Thunk__Instrument_Store, __RaceDetector__Instrument_Store);
	HotpatchInstrumentationFunctionsFast.Emplace(__Thunk__Instrument_Load, __RaceDetector__Instrument_Load);
	HotpatchInstrumentationFunctionsFast.Emplace(__Thunk__Instrument_VPtr_Store, __RaceDetector__Instrument_VPtr_Store);
	HotpatchInstrumentationFunctionsFast.Emplace(__Thunk__Instrument_VPtr_Load, __RaceDetector__Instrument_VPtr_Load);

	// In non-monolithic, we also hook the core functions directly instead of via the thunks to save on another set of JMPs
#if !IS_MONOLITHIC
	AlwaysOnInstrumentationFunctions.Emplace(__Instrument_FuncEntry, __RaceDetector__Instrument_FuncEntry);
	AlwaysOnInstrumentationFunctions.Emplace(__Instrument_FuncExit, __RaceDetector__Instrument_FuncExit);

	HotpatchInstrumentationFunctionsFast.Emplace(AnnotateHappensBefore, __RaceDetector__AnnotateHappensBefore);
	HotpatchInstrumentationFunctionsFast.Emplace(AnnotateHappensAfter, __RaceDetector__AnnotateHappensAfter);
	HotpatchInstrumentationFunctionsFast.Emplace(__Instrument_StoreRange, __RaceDetector__Instrument_StoreRange);
	HotpatchInstrumentationFunctionsFast.Emplace(__Instrument_LoadRange, __RaceDetector__Instrument_LoadRange);
	HotpatchInstrumentationFunctionsFast.Emplace(__Instrument_Store, __RaceDetector__Instrument_Store);
	HotpatchInstrumentationFunctionsFast.Emplace(__Instrument_Load, __RaceDetector__Instrument_Load);
	HotpatchInstrumentationFunctionsFast.Emplace(__Instrument_VPtr_Store, __RaceDetector__Instrument_VPtr_Store);
	HotpatchInstrumentationFunctionsFast.Emplace(__Instrument_VPtr_Load, __RaceDetector__Instrument_VPtr_Load);
#endif

#define HOTPATCH_FUNC(Type) \
	HotpatchInstrumentationFunctions.Emplace(__Thunk__Instrument_AtomicLoad_##Type, __RaceDetector__Instrument_AtomicLoad_##Type); \
	HotpatchInstrumentationFunctions.Emplace(__Thunk__Instrument_AtomicStore_##Type, __RaceDetector__Instrument_AtomicStore_##Type); \
	HotpatchInstrumentationFunctions.Emplace(__Thunk__Instrument_AtomicExchange_##Type, __RaceDetector__Instrument_AtomicExchange_##Type); \
	HotpatchInstrumentationFunctions.Emplace(__Thunk__Instrument_AtomicCompareExchange_##Type, __RaceDetector__Instrument_AtomicCompareExchange_##Type);

#define HOTPATCH_RMW_FUNC(Func, Type) \
	HotpatchInstrumentationFunctions.Emplace(__Thunk__Instrument_Atomic##Func##_##Type, __RaceDetector__Instrument_Atomic##Func##_##Type);

	HOTPATCH_FUNC(int8)
	HOTPATCH_FUNC(int16)
	HOTPATCH_FUNC(int32)
	HOTPATCH_FUNC(int64)

	HOTPATCH_RMW_FUNC(FetchAdd, int8)
	HOTPATCH_RMW_FUNC(FetchAdd, int16)
	HOTPATCH_RMW_FUNC(FetchAdd, int32)
	HOTPATCH_RMW_FUNC(FetchAdd, int64)
	HOTPATCH_RMW_FUNC(FetchSub, int8)
	HOTPATCH_RMW_FUNC(FetchSub, int16)
	HOTPATCH_RMW_FUNC(FetchSub, int32)
	HOTPATCH_RMW_FUNC(FetchSub, int64)
	HOTPATCH_RMW_FUNC(FetchOr, int8)
	HOTPATCH_RMW_FUNC(FetchOr, int16)
	HOTPATCH_RMW_FUNC(FetchOr, int32)
	HOTPATCH_RMW_FUNC(FetchOr, int64)
	HOTPATCH_RMW_FUNC(FetchXor, int8)
	HOTPATCH_RMW_FUNC(FetchXor, int16)
	HOTPATCH_RMW_FUNC(FetchXor, int32)
	HOTPATCH_RMW_FUNC(FetchXor, int64)
	HOTPATCH_RMW_FUNC(FetchAnd, int8)
	HOTPATCH_RMW_FUNC(FetchAnd, int16)
	HOTPATCH_RMW_FUNC(FetchAnd, int32)
	HOTPATCH_RMW_FUNC(FetchAnd, int64)
}

INSTRUMENTATION_FUNCTION_ATTRIBUTES void PrepareHotPatchFunctions()
{
	FInstrumentationScope Scope;

	for (const auto [ThunkFn, DetouredFn] : AlwaysOnInstrumentationFunctions)
	{
		Platform::PrepareTrampoline(ThunkFn, DetouredFn, false /* bUseRETBypass */);
	}

	for (const auto [ThunkFn, DetouredFn] : HotpatchInstrumentationFunctions)
	{
		Platform::PrepareTrampoline(ThunkFn, DetouredFn, false /* bUseRETBypass */);
	}

	for (const auto [ThunkFn, DetouredFn] : HotpatchInstrumentationFunctionsFast)
	{
		Platform::PrepareTrampoline(ThunkFn, DetouredFn, true /* bUseRETBypass */);
	}
}

INSTRUMENTATION_FUNCTION_ATTRIBUTES void CleanupHotPatchFunctions()
{
	FInstrumentationScope Scope;

	for (const auto [ThunkFn, DetouredFn] : AlwaysOnInstrumentationFunctions)
	{
		Platform::CleanupTrampoline(ThunkFn);
	}

	for (const auto [ThunkFn, DetouredFn] : HotpatchInstrumentationFunctions)
	{
		Platform::CleanupTrampoline(ThunkFn);
	}

	for (const auto [ThunkFn, DetouredFn] : HotpatchInstrumentationFunctionsFast)
	{
		Platform::CleanupTrampoline(ThunkFn);
	}
}

INSTRUMENTATION_FUNCTION_ATTRIBUTES void HookAlwaysOnInstrumentation()
{
	FInstrumentationScope Scope;

	for (const auto [ThunkFn, DetouredFn] : AlwaysOnInstrumentationFunctions)
	{
		Platform::ActivateTrampoline(ThunkFn);
	}

	Platform::FlushInstructionCache();
}

INSTRUMENTATION_FUNCTION_ATTRIBUTES void HookInstrumentation()
{
	FInstrumentationScope Scope;

	for (const auto [ThunkFn, DetouredFn] : HotpatchInstrumentationFunctions)
	{
		Platform::ActivateTrampoline(ThunkFn);
	}

	for (const auto [ThunkFn, DetouredFn] : HotpatchInstrumentationFunctionsFast)
	{
		Platform::ActivateTrampoline(ThunkFn);
	}

	Platform::FlushInstructionCache();
}

INSTRUMENTATION_FUNCTION_ATTRIBUTES void UnhookAlwaysOnInstrumentation()
{
	FInstrumentationScope Scope;

	for (const auto [ThunkFn, DetouredFn] : AlwaysOnInstrumentationFunctions)
	{
		Platform::DeactivateTrampoline(ThunkFn, false /* bUseRETBypass */);
	}

	Platform::FlushInstructionCache();
}

INSTRUMENTATION_FUNCTION_ATTRIBUTES void UnhookInstrumentation()
{
	FInstrumentationScope Scope;

	for (const auto [ThunkFn, DetouredFn] : HotpatchInstrumentationFunctions)
	{
		Platform::DeactivateTrampoline(ThunkFn, false /* bUseRETBypass */);
	}

	for (const auto [ThunkFn, DetouredFn] : HotpatchInstrumentationFunctionsFast)
	{
		Platform::DeactivateTrampoline(ThunkFn, true /* bUseRETBypass */);
	}

	Platform::FlushInstructionCache();
}

INSTRUMENTATION_FUNCTION_ATTRIBUTES void InitializeBoot()
{
	ContextTLSIndex = Platform::AllocTlsSlot();
	ContextTLSValue = 0x1480lu + (unsigned long)ContextTLSIndex * sizeof(void*);
	EnsureCurrentContext();

	// Avoid doing that while reporting a race as it may cause reentrancy issues because of tracing.
	FPlatformStackWalk::InitStackWalking();
}

} // UE::Sanitizer::RaceDetector

#ifdef INSTRUMENTATION_DEBUG
UE_ENABLE_OPTIMIZATION
#endif

#endif // USING_INSTRUMENTATION