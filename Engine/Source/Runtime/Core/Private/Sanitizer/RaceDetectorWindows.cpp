// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/CoreMiscDefines.h"

#if PLATFORM_WINDOWS && USING_INSTRUMENTATION

#include "Sanitizer/Types.h"
#include "Sanitizer/RaceDetector.h"
#include "Sanitizer/RaceDetectorTypes.h"
#include "Sanitizer/RaceDetectorInterface.h"
#include "Instrumentation/Defines.h"
#include "Instrumentation/Containers.h"

#include "Windows/WindowsHWrapper.h"
#include "Windows/AllowWindowsPlatformTypes.h"

THIRD_PARTY_INCLUDES_START
#include <detours/detours.h>
#include <winternl.h> // for NTSTATUS
#include <process.h>  // for _beginthreadex
THIRD_PARTY_INCLUDES_END

using namespace UE::Sanitizer;
using namespace UE::Sanitizer::RaceDetector;

namespace UE::Sanitizer::RaceDetector {
	extern void InitializeBoot();
	extern void SanitizerThreadRun(volatile bool& bContinue);
	extern TRefCountPtr<FContext> GetContextByThreadId(uint32 InThreadId);
	extern volatile bool bIsResettingShadow;
	extern thread_local bool bIsResettingShadowThread;
	extern void WriteToLog(FString&& Message);
	extern bool bRuntimeInitialized;
	constexpr int32 MinStackSize = 256 * 1024;
}

// Sentinel used to validate our TLS walking assumptions at runtime.
static __declspec(thread) volatile uint8 TlsValidationSentinel = 0;

// Tri-state: 0 = not yet checked, 1 = validated OK, -1 = validation failed.
static volatile int8 TlsCleanupState = 0;

// Walks every loaded module's PE TLS directory and invokes the callback with the
// base address and size of each per-thread TLS block. No API calls are made —
// everything is read directly from TEB/PEB/PE structures to avoid reentrancy
// through detoured functions.
// Returns false if the TEB or PEB structures couldn't be read.
template<typename CallbackType>
INSTRUMENTATION_FUNCTION_ATTRIBUTES static bool ForEachTlsRegion(CallbackType&& Callback)
{
	// TEB offset 0x58 (x64) is ThreadLocalStoragePointer
	void** TlsArray = *(void***)((BYTE*)NtCurrentTeb() + 0x58);
	if (!TlsArray)
	{
		return false;
	}

	PEB* Peb = NtCurrentTeb()->ProcessEnvironmentBlock;
	if (!Peb || !Peb->Ldr)
	{
		return false;
	}

	LIST_ENTRY* Head = &Peb->Ldr->InMemoryOrderModuleList;
	for (LIST_ENTRY* Entry = Head->Flink; Entry != Head; Entry = Entry->Flink)
	{
		LDR_DATA_TABLE_ENTRY* LdrEntry = CONTAINING_RECORD(Entry, LDR_DATA_TABLE_ENTRY, InMemoryOrderLinks);
		BYTE* DllBase = (BYTE*)LdrEntry->DllBase;
		if (!DllBase) 
		{
			continue;
		}

		IMAGE_DOS_HEADER* DosHeader = (IMAGE_DOS_HEADER*)DllBase;
		if (DosHeader->e_magic != IMAGE_DOS_SIGNATURE) 
		{
			continue;
		}
		IMAGE_NT_HEADERS64* NtHeaders = (IMAGE_NT_HEADERS64*)(DllBase + DosHeader->e_lfanew);
		if (NtHeaders->Signature != IMAGE_NT_SIGNATURE) 
		{
			continue;
		}

		IMAGE_DATA_DIRECTORY& TlsDataDir = NtHeaders->OptionalHeader.DataDirectory[IMAGE_DIRECTORY_ENTRY_TLS];
		if (TlsDataDir.VirtualAddress == 0) 
		{
			continue;
		}

		IMAGE_TLS_DIRECTORY* TlsDir = (IMAGE_TLS_DIRECTORY*)(DllBase + TlsDataDir.VirtualAddress);
		DWORD TlsIndex = *(DWORD*)TlsDir->AddressOfIndex;
		SIZE_T TlsSize = TlsDir->EndAddressOfRawData - TlsDir->StartAddressOfRawData;
		void* TlsBase = TlsArray[TlsIndex];

		if (TlsBase && TlsSize > 0)
		{
			Callback(TlsBase, TlsSize);
		}
	}

	return true;
}

// Validates that ForEachTlsRegion produces correct results by checking that
// a known __declspec(thread) variable falls within one of the discovered regions.
INSTRUMENTATION_FUNCTION_ATTRIBUTES static bool ValidateTlsCleanupAssumptions()
{
	const BYTE* SentinelAddr = (const BYTE*)&TlsValidationSentinel;
	bool bFound = false;

	bool bSuccess = ForEachTlsRegion([&](void* TlsBase, SIZE_T TlsSize)
	{
		const BYTE* Base = (const BYTE*)TlsBase;
		if (SentinelAddr >= Base && SentinelAddr < Base + TlsSize)
		{
			bFound = true;
		}
	});

	return bSuccess && bFound;
}

// Ensures TLS cleanup has been validated. Called lazily on the first thread
// shutdown so that all modules are loaded and TLS is fully initialized.
INSTRUMENTATION_FUNCTION_ATTRIBUTES static bool EnsureTlsCleanupValidated()
{
	if (TlsCleanupState == 0)
	{
		if (ValidateTlsCleanupAssumptions())
		{
			TlsCleanupState = 1;
		}
		else
		{
			TlsCleanupState = -1;
			UE_LOGF(LogRaceDetector, Warning,
				"TLS shadow cleanup disabled: could not validate TEB/PE TLS assumptions on this OS version. " "TLS-related false positives may occur.");
		}
	}
	return TlsCleanupState > 0;
}

#define START_WINAPI_INSTRUMENTATION(DoIfNotInstrumenting) \
	DWORD OriginalError = GetLastError(); \
	FContext* ContextPtr = GetThreadContext(); \
	if (!FContext::IsValid(ContextPtr) || !ShouldInstrument(*ContextPtr)) \
	{ \
		SetLastError(OriginalError); \
		DoIfNotInstrumenting; \
	} \
	FContext& Context = *ContextPtr; \
	Context.WinInstrumentationDepth++; \
	SetLastError(OriginalError);

#define FINISH_WINAPI_INSTRUMENTATION() \
	Context.WinInstrumentationDepth--;

struct FInstrumentedStartThreadArgs {
	SAFE_OPERATOR_NEW_DELETE()

	LPVOID RealThreadParameter;
	LPTHREAD_START_ROUTINE RealStartRoutine;
	bool bDetailedLog = false;

	// ClockBank inherited from the CreateThread call
	FClockBank ClockBank;

	// Can't use std::atomic here because it causes reentrancy in the instrumentation
	// during thread shutdown and ends up recreating context we're trying to delete.
	volatile HANDLE ThreadHandle{ nullptr };
};

// This is actually really important since we need to set back the same
// Windows error that came back from the true Windows API call since the
// instrumentation is doing some API calls of its own, we can end up
// resetting the last errors and causing issues in perceived results.
struct FLastErrorPreservationScope
{
	DWORD OriginalError;

	INSTRUMENTATION_FUNCTION_ATTRIBUTES FLastErrorPreservationScope()
		: OriginalError(GetLastError())
	{
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES ~FLastErrorPreservationScope()
	{
		SetLastError(OriginalError);
	}
};

// Publish clock, so that if somebody waits on the thread handle, they can acquire the clock
// to establish an happen before/after relationship.
// Must be called from thread about to end.
INSTRUMENTATION_FUNCTION_ATTRIBUTES void InstrumentEndThread()
{
	// Important to guard against trying to access TLS if the runtime has shut down
	// otherwise we might end up with use-after-free during some thread cleanup 
	// at application exit.
	if (!bRuntimeInitialized)
	{
		return;
	}

	HANDLE Handle = nullptr;
	if (FContext* ContextPtr = GetThreadContext(); FContext::IsValid(ContextPtr))
	{
		FInstrumentedStartThreadArgs* ThreadArgs = (FInstrumentedStartThreadArgs*)ContextPtr->ThreadArgs;
		if (ThreadArgs)
		{
			// This can only be nullptr if this thread was so quick to execute
			// that the parent thread didn't have a chance to broadcast the
			// thread handle yet.
			while ((Handle = ThreadArgs->ThreadHandle) == nullptr)
			{
				FPlatformProcess::YieldThread();
			}
			delete ThreadArgs;
			ContextPtr->ThreadArgs = nullptr;
		}
	}

	START_WINAPI_INSTRUMENTATION(ReleaseCurrentContext(); return);
	if (Handle)
	{
		FSyncObjectRef Sync = GetSyncObject(Context, Handle);
		Sync->SyncReleaseAsSoleOwner(Context, _ReturnAddress(), Handle, TEXT("InstrumentEndThread"));
		Context.IncrementClock();
	}
	
	// Clean the stack to avoid false positives when it gets reused for another thread.
	ULONG_PTR LowLimit, HighLimit;
	GetCurrentThreadStackLimits(&LowLimit, &HighLimit);
	UE::Sanitizer::RaceDetector::FreeMemoryRange(reinterpret_cast<void*>(LowLimit), HighLimit - LowLimit);

	// Clean TLS to avoid false positives when memory gets reused for another thread.
	// Validated lazily on the first thread shutdown to ensure all modules are loaded.
	if (EnsureTlsCleanupValidated())
	{
		ForEachTlsRegion([](void* TlsBase, SIZE_T TlsSize)
		{
			UE::Sanitizer::RaceDetector::FreeMemoryRange(TlsBase, TlsSize);
		});
	}

	ReleaseCurrentContext();

	// We released our context so we can't decrement the count now but we can't decrement it
	// before releasing the context because it could cause reentrancy.
	//FINISH_WINAPI_INSTRUMENTATION();
}

INSTRUMENTATION_FUNCTION_ATTRIBUTES DWORD WINAPI InstrumentedStartThread(LPVOID Param)
{
	FInstrumentedStartThreadArgs* ThreadArgs = (FInstrumentedStartThreadArgs*)Param;

	FContext& Context = EnsureCurrentContext();
	Context.ThreadArgs = ThreadArgs;
	{
		FInstrumentationScope InstrumentationScope;
		Context.ClockBank.Acquire(ThreadArgs->ClockBank, _ReturnAddress());

		// Do not try to sync if we reach this before the handle is set from the creating thread
		// it means that CREATE_SUSPENDED was not passed as a flag anyway so we don't need any barrier in that case.
		if (ThreadArgs->ThreadHandle && bRuntimeInitialized)
		{
			// This allows us to sync with ResumeThread if the thread was created in the suspended state.
			FSyncObjectRef Sync = GetSyncObject(Context, ThreadArgs->ThreadHandle);
			Sync->SyncAcquireAsSoleOwnerOrReadOwner(Context, _ReturnAddress(), ThreadArgs->ThreadHandle, TEXT("InstrumentedStartThread"));
		}

		// We don't increment the clock here since we don't have any associated
		// contextid yet. Clock is incremented when contextid is reserved on first
		// memory access.
	}

	if (ThreadArgs->bDetailedLog)
	{
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("[%d] Thread starting at clock %d\n"), Context.ThreadId, Context.CurrentClock());
	}

	// Call the real function.
	DWORD Result = ThreadArgs->RealStartRoutine(ThreadArgs->RealThreadParameter);

	InstrumentEndThread();

	return Result;
}

HANDLE (WINAPI* TrueCreateThread)(LPSECURITY_ATTRIBUTES lpThreadAttributes, SIZE_T dwStackSize, LPTHREAD_START_ROUTINE lpStartAddress, LPVOID lpParameter, DWORD dwCreationFlags, LPDWORD lpThreadId);
INSTRUMENTATION_FUNCTION_ATTRIBUTES HANDLE WINAPI DetouredCreateThread(LPSECURITY_ATTRIBUTES lpThreadAttributes, SIZE_T dwStackSize, LPTHREAD_START_ROUTINE lpStartAddress, LPVOID lpParameter, DWORD dwCreationFlags, LPDWORD lpThreadId)
{
	FContext& Context = EnsureCurrentContext();

	// Make sure threads have enough stack for additional instrumentation depth
	if (dwStackSize != 0 && dwStackSize < UE::Sanitizer::RaceDetector::MinStackSize)
	{
		dwStackSize = UE::Sanitizer::RaceDetector::MinStackSize;
	}

	// If ThreadCreationDepth is non-zero here, we're probably coming from a _beginthreadex call 
	// which was already instrumented. 
	// Pass along the arguments as-they-are as they're already the instrumented arguments.
	if (Context.ThreadCreationDepth > 0)
	{
		return TrueCreateThread(lpThreadAttributes, dwStackSize, lpStartAddress, lpParameter, dwCreationFlags, lpThreadId);
	}

	FInstrumentationScope InstrumentationScope;
	if (Context.DetailedLogDepth || bDetailedLogGlobal)
	{
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("[%d] Creating thread (current clock %d)\n"), Context.ThreadId, Context.CurrentClock());
	}

	FInstrumentedStartThreadArgs* Args = new FInstrumentedStartThreadArgs;
	Args->RealThreadParameter = lpParameter;
	Args->RealStartRoutine = lpStartAddress;
	Args->ClockBank = Context.ClockBank;
	Args->bDetailedLog = Context.DetailedLogDepth;
	Context.IncrementClock();

	HANDLE Handle = TrueCreateThread(lpThreadAttributes, dwStackSize, InstrumentedStartThread, Args, dwCreationFlags, lpThreadId);
	if (Handle)
	{
		Args->ThreadHandle = Handle;
	}
	return Handle;
}

UPTRINT (WINAPI* True_beginthreadex)(void* security, unsigned stack_size, unsigned(__stdcall* start_address)(void*), void* arglist, unsigned initflag, unsigned* thrdaddr);
INSTRUMENTATION_FUNCTION_ATTRIBUTES UPTRINT WINAPI Detoured_beginthreadex(void* security, unsigned stack_size, unsigned(__stdcall* start_address)(void*), void* arglist, unsigned initflag, unsigned* thrdaddr)
{
	FContext& Context = EnsureCurrentContext();
	FInstrumentationScope InstrumentationScope;
	if (Context.DetailedLogDepth || bDetailedLogGlobal)
	{
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("[%d] Creating thread (current clock %d)\n"), Context.ThreadId, Context.CurrentClock());
	}

	// Make sure threads have enough stack for additional instrumentation depth
	if (stack_size != 0 && stack_size < UE::Sanitizer::RaceDetector::MinStackSize)
	{
		stack_size = UE::Sanitizer::RaceDetector::MinStackSize;
	}

	FInstrumentedStartThreadArgs* Args = new FInstrumentedStartThreadArgs;
	Args->RealThreadParameter = arglist;
	Args->RealStartRoutine = (LPTHREAD_START_ROUTINE)start_address;
	Args->ClockBank = Context.ClockBank;
	Args->bDetailedLog = Context.DetailedLogDepth;
	Context.IncrementClock();
	
	Context.ThreadCreationDepth++;
	UPTRINT Handle = True_beginthreadex(security, stack_size, (unsigned(__stdcall*)(void*))InstrumentedStartThread, Args, initflag, thrdaddr);
	Context.ThreadCreationDepth--;
	if (Handle)
	{
		Args->ThreadHandle = (HANDLE)Handle;
	}
	return Handle;
}

void (WINAPI* TrueFreeLibraryAndExitThread)(HMODULE hLibModule, DWORD dwExitCode);
INSTRUMENTATION_FUNCTION_ATTRIBUTES void WINAPI DetouredFreeLibraryAndExitThread(HMODULE hLibModule, DWORD dwExitCode)
{
	InstrumentEndThread();
	TrueFreeLibraryAndExitThread(hLibModule, dwExitCode);
}

void(__cdecl* True_Cnd_do_broadcast_at_thread_exit)(void);
INSTRUMENTATION_FUNCTION_ATTRIBUTES void WINAPI Detoured_Cnd_do_broadcast_at_thread_exit()
{
	InstrumentEndThread();
	True_Cnd_do_broadcast_at_thread_exit();
}

void (WINAPI* TrueExitThread)(DWORD dwExitCode);
INSTRUMENTATION_FUNCTION_ATTRIBUTES void WINAPI DetouredExitThread(DWORD dwExitCode)
{
	InstrumentEndThread();
	TrueExitThread(dwExitCode);
}

INSTRUMENTATION_FUNCTION_ATTRIBUTES void AcquireWaitHandle(HANDLE Handle, void* ReturnAddress, const TCHAR* OpName)
{
	FContext& Context = EnsureCurrentContext();

	FSyncObjectRef Sync = GetSyncObject(Context, Handle);
	Sync->SyncAcquire(Context, []() {}, ReturnAddress, Handle, OpName);
	Context.IncrementClock();
}

DWORD(WINAPI* TrueWaitForSingleObject)(HANDLE hHandle, DWORD dwMilliseconds) = WaitForSingleObject;
DWORD(WINAPI* TrueWaitForSingleObjectEx)(HANDLE hHandle, DWORD dwMilliseconds, BOOL bAlertable) = WaitForSingleObjectEx;

INSTRUMENTATION_FUNCTION_ATTRIBUTES DWORD WINAPI DetouredWaitForSingleObject(HANDLE hHandle, DWORD dwMilliseconds)
{
	// Use the Ex version here to avoid recursing into our other detoured function
	DWORD Result = TrueWaitForSingleObjectEx(hHandle, dwMilliseconds, FALSE);

	START_WINAPI_INSTRUMENTATION(return Result);
	FLastErrorPreservationScope LastErrorScope;
	if (Result == WAIT_OBJECT_0)
	{
		AcquireWaitHandle(hHandle, _ReturnAddress(), TEXT("WaitForSingleObject"));
	}

	FINISH_WINAPI_INSTRUMENTATION();
	return Result;
}

INSTRUMENTATION_FUNCTION_ATTRIBUTES DWORD WINAPI DetouredWaitForSingleObjectEx(HANDLE hHandle, DWORD dwMilliseconds, BOOL bAlertable)
{
	DWORD Result = TrueWaitForSingleObjectEx(hHandle, dwMilliseconds, bAlertable);

	START_WINAPI_INSTRUMENTATION(return Result);
	FLastErrorPreservationScope LastErrorScope;

	if (Result == WAIT_OBJECT_0)
	{
		AcquireWaitHandle(hHandle, _ReturnAddress(), TEXT("WaitForSingleObjectEx"));
	}

	FINISH_WINAPI_INSTRUMENTATION();
	return Result;
}

BOOL(WINAPI* TrueSetEvent)(HANDLE hHandle) = SetEvent;
INSTRUMENTATION_FUNCTION_ATTRIBUTES BOOL WINAPI DetouredSetEvent(HANDLE hHandle)
{
	START_WINAPI_INSTRUMENTATION(return TrueSetEvent(hHandle));

	// The scope is important to make sure the TRefCountPtr is destroyed before we FINISH_WINAPI_INSTRUMENTATION.
	{
		FSyncObjectRef Sync = GetSyncObject(Context, hHandle);
		Sync->SyncRelease(Context, []() {}, _ReturnAddress(), hHandle, TEXT("SetEvent"));
		Context.IncrementClock();
	}
	BOOL Result = TrueSetEvent(hHandle);

	FINISH_WINAPI_INSTRUMENTATION();
	return Result;
}

BOOL(WINAPI* TrueWaitOnAddress)(volatile VOID* Address, PVOID CompareAddress, SIZE_T AddressSize, DWORD dwMilliseconds) = WaitOnAddress;
INSTRUMENTATION_FUNCTION_ATTRIBUTES BOOL WINAPI DetouredWaitOnAddress(volatile VOID* Address, PVOID CompareAddress, SIZE_T AddressSize, DWORD dwMilliseconds)
{
	BOOL Result = TrueWaitOnAddress(Address, CompareAddress, AddressSize, dwMilliseconds);

	START_WINAPI_INSTRUMENTATION(return Result);
	FLastErrorPreservationScope LastErrorScope;

	if (Result)
	{
		// Prevent spurious wake ups from providing a barrier by making sure the value has changed
		const uint8* Ptr1 = (const uint8*)Address;
		const uint8* Ptr2 = (const uint8*)CompareAddress;

		bool bIsValueChanged = false;
		for (SIZE_T Index = 0; Index < AddressSize; ++Index)
		{
			if (Ptr1[Index] != Ptr2[Index])
			{
				bIsValueChanged = true;
				break;
			}
		}

		if (bIsValueChanged)
		{
			FSyncObjectRef Sync = GetSyncObject(Context, (void*)Address);
			Sync->SyncAcquire(Context, []() {}, _ReturnAddress(), (void*)Address, TEXT("WaitOnAddress"));
			Context.IncrementClock();
		}
	}

	FINISH_WINAPI_INSTRUMENTATION();
	return Result;
}

void (WINAPI* TrueWakeByAddressSingle)(PVOID Address) = WakeByAddressSingle;
INSTRUMENTATION_FUNCTION_ATTRIBUTES void WINAPI DetouredWakeByAddressSingle(PVOID Address)
{
	START_WINAPI_INSTRUMENTATION(TrueWakeByAddressSingle(Address); return);

	{
		FLastErrorPreservationScope LastErrorScope;
		FSyncObjectRef Sync = GetSyncObject(Context, Address);
		Sync->SyncRelease(Context, []() {}, _ReturnAddress(), Address, TEXT("WakeByAddressSingle"));
		Context.IncrementClock();
	}

	TrueWakeByAddressSingle(Address);

	FINISH_WINAPI_INSTRUMENTATION();
}


BOOL(WINAPI* TrueTryEnterCriticalSection)(CRITICAL_SECTION* lpCriticalSection) = TryEnterCriticalSection;
INSTRUMENTATION_FUNCTION_ATTRIBUTES BOOL WINAPI DetouredTryEnterCriticalSection(CRITICAL_SECTION* lpCriticalSection)
{
	BOOL Result = TrueTryEnterCriticalSection(lpCriticalSection);

	START_WINAPI_INSTRUMENTATION(return Result);
	FLastErrorPreservationScope LastErrorScope;

	// If we entered the critical section, we are now the sole owner.
	if (Result)
	{
		FSyncObjectRef Sync = GetSyncObject(Context, lpCriticalSection);
		Sync->SyncAcquireAsSoleOwnerOrReadOwner(Context, _ReturnAddress(), lpCriticalSection, TEXT("TryEnterCriticalSection"));
		Context.IncrementClock();
	}

	FINISH_WINAPI_INSTRUMENTATION();
	return Result;
}

void(WINAPI* TrueEnterCriticalSection)(CRITICAL_SECTION* lpCriticalSection) = EnterCriticalSection;
INSTRUMENTATION_FUNCTION_ATTRIBUTES void WINAPI DetouredEnterCriticalSection(CRITICAL_SECTION* lpCriticalSection)
{
	TrueEnterCriticalSection(lpCriticalSection);

	START_WINAPI_INSTRUMENTATION(return);
	FLastErrorPreservationScope LastErrorScope;

	// The scope is important to make sure the TRefCountPtr is destroyed before we FINISH_WINAPI_INSTRUMENTATION.
	{
		// We are now the sole owner.
		FSyncObjectRef Sync = GetSyncObject(Context, lpCriticalSection);
		Sync->SyncAcquireAsSoleOwnerOrReadOwner(Context, _ReturnAddress(), lpCriticalSection, TEXT("EnterCriticalSection"));
		Context.IncrementClock();
	}

	FINISH_WINAPI_INSTRUMENTATION();
}

void (WINAPI* TrueLeaveCriticalSection)(CRITICAL_SECTION* lpCriticalSection) = LeaveCriticalSection;
INSTRUMENTATION_FUNCTION_ATTRIBUTES void WINAPI DetouredLeaveCriticalSection(CRITICAL_SECTION* lpCriticalSection)
{
	START_WINAPI_INSTRUMENTATION(TrueLeaveCriticalSection(lpCriticalSection); return);

	{
		FLastErrorPreservationScope LastErrorScope;
		// Assume this is called with the right semantics, i.e. that we are actual
		// owners of the critical section.
		FSyncObjectRef Sync = GetSyncObject(Context, lpCriticalSection);
		Sync->SyncReleaseAsSoleOwner(Context, _ReturnAddress(), lpCriticalSection, TEXT("LeaveCriticalSection"));
		Context.IncrementClock();
	}
	TrueLeaveCriticalSection(lpCriticalSection);

	FINISH_WINAPI_INSTRUMENTATION();
}

// SRW locks.
void (WINAPI* TrueAcquireSRWLockShared)(PSRWLOCK SRWLock) = AcquireSRWLockShared;
INSTRUMENTATION_FUNCTION_ATTRIBUTES void WINAPI DetouredAcquireSRWLockShared(PSRWLOCK SRWLock)
{
	TrueAcquireSRWLockShared(SRWLock);

	START_WINAPI_INSTRUMENTATION(return);
	FLastErrorPreservationScope LastErrorScope;

	// The scope is important to make sure the TRefCountPtr is destroyed before we FINISH_WINAPI_INSTRUMENTATION.
	{
		FSyncObjectRef Sync = GetSyncObject(Context, SRWLock);
		Sync->SyncAcquireAsSoleOwnerOrReadOwner(Context, _ReturnAddress(), SRWLock, TEXT("AcquireSRWLockShared"));
		Context.IncrementClock();
	}

	FINISH_WINAPI_INSTRUMENTATION();
}

BOOLEAN(WINAPI* TrueTryAcquireSRWLockShared)(PSRWLOCK SRWLock) = TryAcquireSRWLockShared;
INSTRUMENTATION_FUNCTION_ATTRIBUTES BOOLEAN WINAPI DetouredTryAcquireSRWLockShared(PSRWLOCK SRWLock)
{
	BOOLEAN Result = TrueTryAcquireSRWLockShared(SRWLock);
	
	START_WINAPI_INSTRUMENTATION(return Result);

	FLastErrorPreservationScope LastErrorScope;
	if (Result)
	{
		FSyncObjectRef Sync = GetSyncObject(Context, SRWLock);
		Sync->SyncAcquireAsSoleOwnerOrReadOwner(Context, _ReturnAddress(), SRWLock, TEXT("TryAcquireSRWLockShared"));
		Context.IncrementClock();
	}

	FINISH_WINAPI_INSTRUMENTATION();
	return Result;
}

void (WINAPI* TrueReleaseSRWLockShared)(PSRWLOCK SRWLock) = ReleaseSRWLockShared;
INSTRUMENTATION_FUNCTION_ATTRIBUTES void WINAPI DetouredReleaseSRWLockShared(PSRWLOCK SRWLock)
{
	START_WINAPI_INSTRUMENTATION(TrueReleaseSRWLockShared(SRWLock); return);

	{
		FLastErrorPreservationScope LastErrorScope;
		FSyncObjectRef Sync = GetSyncObject(Context, SRWLock);
		Sync->SyncRelease(Context, []() {}, _ReturnAddress(), SRWLock, TEXT("ReleaseSRWLockShared"));
		Context.IncrementClock();
	}

	TrueReleaseSRWLockShared(SRWLock);

	FINISH_WINAPI_INSTRUMENTATION();
}

void (WINAPI* TrueAcquireSRWLockExclusive)(PSRWLOCK SRWLock) = AcquireSRWLockExclusive;
INSTRUMENTATION_FUNCTION_ATTRIBUTES void WINAPI DetouredAcquireSRWLockExclusive(PSRWLOCK SRWLock)
{
	TrueAcquireSRWLockExclusive(SRWLock);

	START_WINAPI_INSTRUMENTATION(return);
	FLastErrorPreservationScope LastErrorScope;

	// The scope is important to make sure the TRefCountPtr is destroyed before we FINISH_WINAPI_INSTRUMENTATION.
	{
		FSyncObjectRef Sync = GetSyncObject(Context, SRWLock);
		Sync->SyncAcquireAsSoleOwnerOrReadOwner(Context, _ReturnAddress(), SRWLock, TEXT("AcquireSRWLockExclusive"));
		Context.IncrementClock();
	}

	FINISH_WINAPI_INSTRUMENTATION();
}

BOOLEAN(WINAPI* TrueTryAcquireSRWLockExclusive)(PSRWLOCK SRWLock) = TryAcquireSRWLockExclusive;
INSTRUMENTATION_FUNCTION_ATTRIBUTES BOOLEAN WINAPI DetouredTryAcquireSRWLockExclusive(PSRWLOCK SRWLock)
{
	BOOLEAN Result = TrueTryAcquireSRWLockExclusive(SRWLock);

	START_WINAPI_INSTRUMENTATION(return Result);
	FLastErrorPreservationScope LastErrorScope;

	if (Result)
	{
		FSyncObjectRef Sync = GetSyncObject(Context, SRWLock);
		Sync->SyncAcquireAsSoleOwnerOrReadOwner(Context, _ReturnAddress(), SRWLock, TEXT("TryAcquireSRWLockExclusive"));
		Context.IncrementClock();
	}

	FINISH_WINAPI_INSTRUMENTATION();
	return Result;
}

void (WINAPI* TrueReleaseSRWLockExclusive)(PSRWLOCK SRWLock) = ReleaseSRWLockExclusive;
INSTRUMENTATION_FUNCTION_ATTRIBUTES void WINAPI DetouredReleaseSRWLockExclusive(PSRWLOCK SRWLock)
{
	START_WINAPI_INSTRUMENTATION(TrueReleaseSRWLockExclusive(SRWLock); return);

	{
		FLastErrorPreservationScope LastErrorScope;
		FSyncObjectRef Sync = GetSyncObject(Context, SRWLock);
		Sync->SyncReleaseAsSoleOwner(Context, _ReturnAddress(), SRWLock, TEXT("ReleaseSRWLockExclusive"));
		Context.IncrementClock();
	}

	TrueReleaseSRWLockExclusive(SRWLock);

	FINISH_WINAPI_INSTRUMENTATION();
}

typedef PVOID (STDAPICALLTYPE RTLALLOCATEHEAP)(
	[in]           PVOID  HeapHandle,
	[in, optional] ULONG  Flags,
	[in]           SIZE_T Size
);

RTLALLOCATEHEAP* TrueRtlAllocateHeap = nullptr;
INSTRUMENTATION_FUNCTION_ATTRIBUTES LPVOID STDAPICALLTYPE DetouredRtlAllocateHeap(HANDLE hHeap, DWORD dwFlags, SIZE_T dwBytes)
{
	START_WINAPI_INSTRUMENTATION(return TrueRtlAllocateHeap(hHeap, dwFlags, dwBytes););
	LPVOID Result = TrueRtlAllocateHeap(hHeap, dwFlags, dwBytes);
	FINISH_WINAPI_INSTRUMENTATION();
	return Result;
}

typedef NTSTATUS(STDAPICALLTYPE RTLFREEHEAP)
(
	IN PVOID HeapHandle,
	IN ULONG Flags,								/* optional */
	IN PVOID MemoryPointer
);

RTLFREEHEAP* TrueRtlFreeHeap = nullptr;
INSTRUMENTATION_FUNCTION_ATTRIBUTES NTSTATUS STDAPICALLTYPE DetouredRtlFreeHeap(HANDLE hHeap, DWORD dwFlags, LPVOID lpMem)
{
	START_WINAPI_INSTRUMENTATION(return TrueRtlFreeHeap(hHeap, dwFlags, lpMem););

	SIZE_T Size = 0;
	if (lpMem)
	{
		Size = HeapSize(hHeap, dwFlags & HEAP_NO_SERIALIZE, lpMem);
	}
	
	// Free the memory range before freeing the memory to avoid
	// another thread racing with the same address range.
	if (lpMem && Size)
	{
		FreeMemoryRange(lpMem, Size);
	}

	NTSTATUS Result = TrueRtlFreeHeap(hHeap, dwFlags, lpMem);

	FINISH_WINAPI_INSTRUMENTATION();

	return Result;
}

typedef NTSYSAPI PVOID (NTAPI RTLREALLOCATEHEAP)(
	IN PVOID                HeapHandle,
	IN ULONG                Flags,
	IN PVOID                MemoryPointer,
	IN ULONG                Size);

RTLREALLOCATEHEAP* TrueRtlReallocateHeap = nullptr;
INSTRUMENTATION_FUNCTION_ATTRIBUTES PVOID NTAPI DetouredRtlReallocateHeap(PVOID hHeap, ULONG dwFlags, PVOID lpMem, ULONG dwBytes)
{
	// Some calls coming to RtlReallocateHeap are actually well inside the deeper functions of RtlAllocateHeap itself.
	// It appears that we can cause recursion and/or corruption if we try to instrument those so just ignore them
	// since these flags are not documented, it's very unlikely that a call using this flag would come from
	// user facing code that we want to instrument.
	if (dwFlags & 0x800000)
	{
		return TrueRtlReallocateHeap(hHeap, dwFlags, lpMem, dwBytes);
	}

	START_WINAPI_INSTRUMENTATION(return TrueRtlReallocateHeap(hHeap, dwFlags, lpMem, dwBytes));

	if (dwFlags & HEAP_REALLOC_IN_PLACE_ONLY)
	{
		// If we reduce the size, technically another thread could come-in and claim the part
        	// of the allocation that we got rid of, so free it before as it's very unlikely
		// that RtlReallocateHeap would fail when trying to reduce the size of an alloc in place.
		// Worst case, it would be possible to miss a race condition in that zone if the reallocate fails
		// and the application still uses that memory after, but that is very unlikely anyway.
		if (lpMem)
		{
			const SIZE_T OldSize = HeapSize(hHeap, dwFlags & HEAP_NO_SERIALIZE, lpMem);

			if (OldSize > dwBytes)
			{
				FreeMemoryRange((void*)((UPTRINT)lpMem + dwBytes), OldSize - dwBytes);
			}
		}

		return TrueRtlReallocateHeap(hHeap, dwFlags, lpMem, dwBytes);
	}

	// We have to always allocate new blocs in order to invalidate the old memory range before it can get reused.
	void* NewPtr = TrueRtlAllocateHeap(hHeap, dwFlags, dwBytes);

	if (lpMem)
	{
		const SIZE_T OldSize = HeapSize(hHeap, dwFlags & HEAP_NO_SERIALIZE, lpMem);
		if (OldSize > 0)
		{
			if (NewPtr)
			{
				FMemory::Memcpy(NewPtr, lpMem, FMath::Min((ULONG)OldSize, dwBytes));
			}

			FreeMemoryRange(lpMem, OldSize);
		}
		TrueRtlFreeHeap(hHeap, dwFlags, lpMem);
	}
	
	FINISH_WINAPI_INSTRUMENTATION();

	return NewPtr;
}

// NtUnmapViewOfSectionEx is the low-level function used by both UnmapViewOfFile and the DLL
// loader (FreeLibrary -> LdrUnloadDll). When virtual memory is unmapped, the address range
// returns to the OS and may be reused for new allocations. Without shadow cleanup, stale
// entries from the old mapping cause false positive race reports.
// See UE-368317: ReadLibraryImportsFromMemory maps DLLs via MapViewOfFile to parse imports,
// then unmaps them. The shadow entries from those reads trip when the VA is reused.
typedef NTSTATUS(NTAPI NTUNMAPVIEWOFSECTIONEX)(HANDLE ProcessHandle, PVOID BaseAddress, ULONG Flags);
NTUNMAPVIEWOFSECTIONEX* TrueNtUnmapViewOfSectionEx = nullptr;
INSTRUMENTATION_FUNCTION_ATTRIBUTES NTSTATUS NTAPI DetouredNtUnmapViewOfSectionEx(HANDLE ProcessHandle, PVOID BaseAddress, ULONG Flags)
{
	// Only clean shadow when unmapping from the current process.
	if (BaseAddress && ProcessHandle == GetCurrentProcess())
	{
		START_WINAPI_INSTRUMENTATION(return TrueNtUnmapViewOfSectionEx(ProcessHandle, BaseAddress, Flags));

		MEMORY_BASIC_INFORMATION MemInfo;
		const uint8* Addr = (const uint8*)BaseAddress;
		while (VirtualQuery(Addr, &MemInfo, sizeof(MemInfo)) && MemInfo.AllocationBase == BaseAddress)
		{
			if (MemInfo.State == MEM_COMMIT)
			{
				FreeMemoryRange((void*)Addr, MemInfo.RegionSize);
			}
			Addr += MemInfo.RegionSize;
		}

		NTSTATUS Result = TrueNtUnmapViewOfSectionEx(ProcessHandle, BaseAddress, Flags);

		FINISH_WINAPI_INSTRUMENTATION();
		return Result;
	}

	return TrueNtUnmapViewOfSectionEx(ProcessHandle, BaseAddress, Flags);
}

DWORD SuspendThread(
  [in] HANDLE hThread
);

typedef NTSYSAPI DWORD(NTAPI SUSPENDTHREAD)(IN HANDLE ThreadHandle);

SUSPENDTHREAD* TrueSuspendThread = nullptr;
INSTRUMENTATION_FUNCTION_ATTRIBUTES DWORD NTAPI DetouredSuspendThread(HANDLE ThreadHandle)
{
	START_WINAPI_INSTRUMENTATION(return TrueSuspendThread(ThreadHandle));

	// Suspend the thread first, then acquire its clock state. Once SuspendThread returns
	// successfully, the target thread is stopped and all its prior memory accesses are
	// visible. This is needed for callstack capture patterns where thread A suspends
	// thread B, reads B's context/stack, then resumes B.
	DWORD result = TrueSuspendThread(ThreadHandle);
	if (result != (DWORD)-1)
	{
		DWORD ThreadId = ::GetThreadId(ThreadHandle);
		TRefCountPtr<FContext> SuspendedThreadContext = GetContextByThreadId(ThreadId);
		if (SuspendedThreadContext)
		{
			Context.ClockBank.Acquire(SuspendedThreadContext->ClockBank, _ReturnAddress());
		}
	}

	FINISH_WINAPI_INSTRUMENTATION();

	return result;
}

DWORD ResumeThread(
  [in] HANDLE hThread
);

typedef NTSYSAPI DWORD(NTAPI RESUMETHREAD)(IN HANDLE ThreadHandle);

RESUMETHREAD* TrueResumeThread = nullptr;
INSTRUMENTATION_FUNCTION_ATTRIBUTES DWORD NTAPI DetouredResumeThread(HANDLE ThreadHandle)
{
	START_WINAPI_INSTRUMENTATION(return TrueResumeThread(ThreadHandle));

	// Add a release barrier when thread is resumed to avoid false positive that could happen
	// if something is written between the createthread and the resumethread and then 
	// read by the thread once it is resumed.
	FSyncObjectRef Sync = GetSyncObject(Context, ThreadHandle);
	Sync->SyncReleaseAsSoleOwner(Context, _ReturnAddress(), ThreadHandle, TEXT("ResumeThread"));

	// This handle synchronization of SuspendThread/ResumeThread pairs (e.g. callstack capture)
	// once the thread is already started.
	DWORD ThreadId = ::GetThreadId(ThreadHandle);
	TRefCountPtr<FContext> SuspendedThreadContext = GetContextByThreadId(ThreadId);
	if (SuspendedThreadContext)
	{
		SuspendedThreadContext->ClockBank.Acquire(Context.ClockBank, _ReturnAddress());
	}
	DWORD result = TrueResumeThread(ThreadHandle);

	FINISH_WINAPI_INSTRUMENTATION();

	return result;
}

// Functions we need to detour for race detector.
TArray<TPair<PVOID*, void*>> DetouredFunctions;

INSTRUMENTATION_FUNCTION_ATTRIBUTES void PopulateDetouredFunctions()
{
	if (!DetouredFunctions.IsEmpty())
	{
		return;
	}

	TrueSuspendThread = ::SuspendThread;
	DetouredFunctions.Emplace(&(PVOID&)TrueSuspendThread, DetouredSuspendThread);
	TrueResumeThread = ::ResumeThread;
	DetouredFunctions.Emplace(&(PVOID&)TrueResumeThread, DetouredResumeThread);
	TrueWaitForSingleObject = ::WaitForSingleObject;
	DetouredFunctions.Emplace(&(PVOID&)TrueWaitForSingleObject, DetouredWaitForSingleObject);
	TrueWaitForSingleObjectEx = ::WaitForSingleObjectEx;
	DetouredFunctions.Emplace(&(PVOID&)TrueWaitForSingleObjectEx, DetouredWaitForSingleObjectEx);
	TrueSetEvent = ::SetEvent;
	DetouredFunctions.Emplace(&(PVOID&)TrueSetEvent, DetouredSetEvent);

	TrueWaitOnAddress = ::WaitOnAddress;
	DetouredFunctions.Emplace(&(PVOID&)TrueWaitOnAddress, DetouredWaitOnAddress);
	TrueWakeByAddressSingle = ::WakeByAddressSingle;
	DetouredFunctions.Emplace(&(PVOID&)TrueWakeByAddressSingle, DetouredWakeByAddressSingle);

	TrueLeaveCriticalSection = ::LeaveCriticalSection;
	DetouredFunctions.Emplace(&(PVOID&)TrueLeaveCriticalSection, DetouredLeaveCriticalSection);
	TrueTryEnterCriticalSection = ::TryEnterCriticalSection;
	DetouredFunctions.Emplace(&(PVOID&)TrueTryEnterCriticalSection, DetouredTryEnterCriticalSection);
	TrueEnterCriticalSection = ::EnterCriticalSection;
	DetouredFunctions.Emplace(&(PVOID&)TrueEnterCriticalSection, DetouredEnterCriticalSection);

	TrueAcquireSRWLockShared = ::AcquireSRWLockShared;
	DetouredFunctions.Emplace(&(PVOID&)TrueAcquireSRWLockShared, DetouredAcquireSRWLockShared);
	TrueTryAcquireSRWLockShared = ::TryAcquireSRWLockShared;
	DetouredFunctions.Emplace(&(PVOID&)TrueTryAcquireSRWLockShared, DetouredTryAcquireSRWLockShared);
	TrueReleaseSRWLockShared = ::ReleaseSRWLockShared;
	DetouredFunctions.Emplace(&(PVOID&)TrueReleaseSRWLockShared, DetouredReleaseSRWLockShared);
	TrueAcquireSRWLockExclusive = ::AcquireSRWLockExclusive;
	DetouredFunctions.Emplace(&(PVOID&)TrueAcquireSRWLockExclusive, DetouredAcquireSRWLockExclusive);
	TrueTryAcquireSRWLockExclusive = ::TryAcquireSRWLockExclusive;
	DetouredFunctions.Emplace(&(PVOID&)TrueTryAcquireSRWLockExclusive, DetouredTryAcquireSRWLockExclusive);
	TrueReleaseSRWLockExclusive = ::ReleaseSRWLockExclusive;
	DetouredFunctions.Emplace(&(PVOID&)TrueReleaseSRWLockExclusive, DetouredReleaseSRWLockExclusive);

	// Some allocations go directly through the ansi allocator so we need to instrument them
	// to avoid false positives.
	TrueRtlFreeHeap = (RTLFREEHEAP*)GetProcAddress(GetModuleHandleA("ntdll"), "RtlFreeHeap");
	TrueRtlReallocateHeap = (RTLREALLOCATEHEAP*)GetProcAddress(GetModuleHandleA("ntdll"), "RtlReAllocateHeap");
	TrueRtlAllocateHeap = (RTLALLOCATEHEAP*)GetProcAddress(GetModuleHandleA("ntdll"), "RtlAllocateHeap");

	DetouredFunctions.Emplace(&(PVOID&)TrueRtlFreeHeap, DetouredRtlFreeHeap);
	DetouredFunctions.Emplace(&(PVOID&)TrueRtlReallocateHeap, DetouredRtlReallocateHeap);
	DetouredFunctions.Emplace(&(PVOID&)TrueRtlAllocateHeap, DetouredRtlAllocateHeap);

	// Hook NtUnmapViewOfSectionEx to clean up shadow state when memory-mapped views or
	// DLL image sections are unmapped. Covers both UnmapViewOfFile and FreeLibrary paths.
	TrueNtUnmapViewOfSectionEx = (NTUNMAPVIEWOFSECTIONEX*)GetProcAddress(GetModuleHandleA("ntdll"), "NtUnmapViewOfSectionEx");
	if (TrueNtUnmapViewOfSectionEx)
	{
		DetouredFunctions.Emplace(&(PVOID&)TrueNtUnmapViewOfSectionEx, DetouredNtUnmapViewOfSectionEx);
	}
}

namespace UE::Sanitizer::RaceDetector::Platform {

	UPTRINT ShadowBase = 0;
	UPTRINT ShadowSize = 0;
	UPTRINT ShadowClockBase = 0;
	UPTRINT ShadowEnd = 0;
	UPTRINT ShadowBitmapBase = 0;
	UPTRINT ShadowBitmapSize = 0;
	UPTRINT ShadowBitmapEnd = 0;
	UPTRINT DirtyShadowBitmapBase = 0;
	UPTRINT DirtyShadowBitmapSize = 0;
	UPTRINT DirtyShadowBitmapEnd = 0;
	UPTRINT PageSize = 0;
	UPTRINT PageSizeBitShift = 0;
	UPTRINT PageSizeMask = 0;
	HANDLE  SanitizerThreadHandle = 0;
	volatile bool bHasShadowMemoryMapped = false;
	volatile bool bSanitizerThreadContinue = true;

	INSTRUMENTATION_FUNCTION_ATTRIBUTES DWORD WINAPI SanitizerThreadProc(LPVOID lpParameter)
	{
		SanitizerThreadRun(bSanitizerThreadContinue);
		return 0;
	}
	
	INSTRUMENTATION_FUNCTION_ATTRIBUTES bool HasShadowMemoryMapped()
	{
		return bHasShadowMemoryMapped;
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES void SleepMS(uint32 MilliSeconds)
	{
		Sleep(MilliSeconds);
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES void PrepareTrampoline(void* ThunkFunctionAddress, void* DestFunctionAddress, bool bUseRETBypass)
	{
		UPTRINT ThunkFunctionStart = (UPTRINT)ThunkFunctionAddress;

		uint8* PrefixStart = (uint8*)(ThunkFunctionStart - INSTRUMENTATION_HOTPATCH_PREFIX_NOPS);
		for (int32 Index = 0; Index < INSTRUMENTATION_HOTPATCH_PREFIX_NOPS; ++Index)
		{
			// Verify that we have NOPs to store the trampoline. 
			if (PrefixStart[Index] != 0x90)
			{

				UE_LOGF(LogRaceDetector, Fatal, "The function at address %p doesn't have a patchable prefix or was already patched by another system", ThunkFunctionAddress);
				return;
			}
		}

		uint16* FunctionStart = (uint16*)(ThunkFunctionStart);
		if (*FunctionStart != 0xC3C3 /* RETs */ && *FunctionStart != 0x9066 /* Two byte NOP */)
		{
			UE_LOGF(LogRaceDetector, Fatal, "The function at address %p doesn't have a patchable entry or was already patched by another system", ThunkFunctionAddress);
			return;
		}

		unsigned long OldProtection;
		if (!VirtualProtect(reinterpret_cast<void*>(PrefixStart), INSTRUMENTATION_HOTPATCH_TOTAL_NOPS, PAGE_EXECUTE_READWRITE, &OldProtection))
		{
			UE_LOGF(LogRaceDetector, Fatal, "Unable to change page protection for hotpatching at %p (error %d)", ThunkFunctionAddress, GetLastError());
		}

		// Set up the unconditional absolute jump
		uint8* Trampoline = (uint8*)(ThunkFunctionStart - INSTRUMENTATION_HOTPATCH_PREFIX_NOPS);
		*(uint16*)(Trampoline + 0)  = 0xB848; /* MOV RAX, imm64  */
		*(uint64*)(Trampoline + 2)  = (uint64)DestFunctionAddress;
		*(uint16*)(Trampoline + 10) = 0xE0FF; /* JMP RAX */

		/* Replace NOP by RETs to further improve perf when instructed to do so */
		if (bUseRETBypass)
		{
			*(uint16*)ThunkFunctionStart = 0xC3C3; 
		}

		// Restore old protection
		if (!VirtualProtect(reinterpret_cast<void*>(PrefixStart), INSTRUMENTATION_HOTPATCH_TOTAL_NOPS, OldProtection, &OldProtection))
		{
			UE_LOGF(LogRaceDetector, Fatal, "Unable to restore page protection for hotpatching at %p (error %d)", ThunkFunctionAddress, GetLastError());
		}
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES void CleanupTrampoline(void* ThunkFunctionAddress)
	{
		UPTRINT ThunkFunctionStart = (UPTRINT)ThunkFunctionAddress;

		uint8* PrefixStart = (uint8*)(ThunkFunctionStart - INSTRUMENTATION_HOTPATCH_PREFIX_NOPS);

		unsigned long OldProtection;
		if (!VirtualProtect(PrefixStart, INSTRUMENTATION_HOTPATCH_TOTAL_NOPS, PAGE_EXECUTE_READWRITE, &OldProtection))
		{
			UE_LOGF(LogRaceDetector, Fatal, "Unable to change page protection for hotpatching at %p (error %d)", ThunkFunctionAddress, GetLastError());
		}

		// Make sure there is no jmp left, equivalent to deactivation.
		// This is actually atomic and won't cause any crashes during the transition.
		*(uint16*)ThunkFunctionStart = 0x9066; /* Replace by 2-byte NOP */

		// Restore all the NOPs that were overwritten
		memset(PrefixStart, 0x90, INSTRUMENTATION_HOTPATCH_PREFIX_NOPS);

		// Restore old protection
		if (!VirtualProtect(PrefixStart, INSTRUMENTATION_HOTPATCH_TOTAL_NOPS, OldProtection, &OldProtection))
		{
			UE_LOGF(LogRaceDetector, Fatal, "Unable to restore page protection for hotpatching at %p (error %d)", ThunkFunctionAddress, GetLastError());
		}
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES void ActivateTrampoline(void* ThunkFunctionAddress)
	{
		UPTRINT ThunkFunctionStart = (UPTRINT)ThunkFunctionAddress;

		unsigned long OldProtection;
		if (!VirtualProtect(reinterpret_cast<void*>(ThunkFunctionStart - INSTRUMENTATION_HOTPATCH_PREFIX_NOPS), INSTRUMENTATION_HOTPATCH_TOTAL_NOPS, PAGE_EXECUTE_READWRITE, &OldProtection))
		{
			UE_LOGF(LogRaceDetector, Fatal, "Unable to change page protection for hotpatching at %p (error %d)", ThunkFunctionAddress, GetLastError());
		}

		// Set up a relative jump to the beginning of the prefix section
		// This is actually atomic and won't cause any crashes during the transition.
		uint16 JmpRelativeOperand = 0xFE - INSTRUMENTATION_HOTPATCH_PREFIX_NOPS;
		*(uint16*)ThunkFunctionStart = (uint16)(JmpRelativeOperand << 8) | 0xEB; /* JMP -INSTRUMENTATION_HOTPATCH_PREFIX_NOPS */

		// Restore old protection
		if (!VirtualProtect(reinterpret_cast<void*>(ThunkFunctionStart - INSTRUMENTATION_HOTPATCH_PREFIX_NOPS), INSTRUMENTATION_HOTPATCH_TOTAL_NOPS, OldProtection, &OldProtection))
		{
			UE_LOGF(LogRaceDetector, Fatal, "Unable to restore page protection for hotpatching at %p (error %d)", ThunkFunctionAddress, GetLastError());
		}
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES void DeactivateTrampoline(void* ThunkFunctionAddress, bool bUseRETBypass)
	{
		UPTRINT ThunkFunctionStart = (UPTRINT)ThunkFunctionAddress;

		unsigned long OldProtection;
		if (!VirtualProtect(reinterpret_cast<void*>(ThunkFunctionStart - INSTRUMENTATION_HOTPATCH_PREFIX_NOPS), INSTRUMENTATION_HOTPATCH_TOTAL_NOPS, PAGE_EXECUTE_READWRITE, &OldProtection))
		{
			UE_LOGF(LogRaceDetector, Fatal, "Unable to change page protection for hotpatching at %p (error %d)", ThunkFunctionAddress, GetLastError());
		}

		/* Replace NOP by RETs to further improve perf when instructed to do so */
		if (bUseRETBypass)
		{
			*(uint16*)ThunkFunctionStart = 0xC3C3;
		}
		else
		{
			// This is actually atomic and won't cause any crashes during the transition.
			*(uint16*)ThunkFunctionStart = 0x9066; /* Replace by 2-byte NOP */
		}

		// Restore old protection
		if (!VirtualProtect(reinterpret_cast<void*>(ThunkFunctionStart - INSTRUMENTATION_HOTPATCH_PREFIX_NOPS), INSTRUMENTATION_HOTPATCH_TOTAL_NOPS, OldProtection, &OldProtection))
		{
			UE_LOGF(LogRaceDetector, Fatal, "Unable to restore page protection for hotpatching at %p (error %d)", ThunkFunctionAddress, GetLastError());
		}
	}
	
	INSTRUMENTATION_FUNCTION_ATTRIBUTES void HideFirstChanceExceptionInVisualStudio()
	{
		// Tell MSVC to avoid flooding the debug output with first chance exception
		// since we're going to use an exception handler to commit shadow memory
		// on demand.
		__try
		{
			RaiseException(0x0E0736170, 0, 0, 0);
		}
		__except (EXCEPTION_EXECUTE_HANDLER)
			CA_SUPPRESS(6322)
		{
		}
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES void FlushInstructionCache()
	{
		::FlushInstructionCache(GetCurrentProcess(), 0, 0);
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES uint32 GetCurrentThreadId()
	{
		return ::GetCurrentThreadId();
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES void* GetTlsValue(uint32 Index)
	{
		return ::TlsGetValue(Index);
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES void SetTlsValue(uint32 Index, void* Value)
	{
		::TlsSetValue(Index, Value);
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES uint32 AllocTlsSlot()
	{
		return ::TlsAlloc();
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES void FreeTlsSlot(uint32 Index)
	{
		::TlsFree(Index);
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES void AsymmetricThreadFenceHeavy()
	{
		// The function generates an interprocessor interrupt (IPI) to all processors that are part of the current process affinity.
		// It guarantees the visibility of write operations performed on one processor to the other processors.
		FlushProcessWriteBuffers();
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES bool IsDebuggerPresent()
	{
		return ::IsDebuggerPresent();
	}
	
	INSTRUMENTATION_FUNCTION_ATTRIBUTES void GetCurrentThreadStackLimits(void** LowLimit, void** HighLimit)
	{
		::GetCurrentThreadStackLimits(reinterpret_cast<PULONG_PTR>(LowLimit), reinterpret_cast<PULONG_PTR>(HighLimit));
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES bool IsThreadAlive(uint32 ThreadId)
	{
		HANDLE ThreadHandle = ::OpenThread(THREAD_QUERY_LIMITED_INFORMATION, FALSE, ThreadId);
		if (ThreadHandle != NULL)
		{
			DWORD ExitCode = 0;
			BOOL bResult = ::GetExitCodeThread(ThreadHandle, &ExitCode);
			::CloseHandle(ThreadHandle);
			return bResult && ExitCode == STILL_ACTIVE;
		}

		return false;
	}

	// ------------------------------------------------------------------------------
	// Linear allocator for FSyncObject and FSyncObjectBank.
	// Reserves a large virtual address range upfront, commits pages in chunks on demand,
	// and decommits the entire region in a single syscall on shadow reset.
	// This replaces per-object VirtualAlloc/VirtualFree which is extremely slow at scale.
	// ------------------------------------------------------------------------------
	static uint8*    SyncAllocBase = nullptr;
	static SIZE_T    SyncAllocReservedSize = 0;
	static volatile int64 SyncAllocCommittedSize = 0;
	static volatile int64 SyncAllocOffset = 0;
	static SIZE_T    SyncAllocCommitGranularity = 0;
	static volatile int32 SyncAllocGlobalRefCount = 0;

	INSTRUMENTATION_FUNCTION_ATTRIBUTES static void SyncAllocEnsureCommitted(SIZE_T RequiredSize)
	{
		while ((SIZE_T)SyncAllocCommittedSize < RequiredSize)
		{
			int64 OldCommitted = SyncAllocCommittedSize;
			if ((SIZE_T)OldCommitted >= RequiredSize)
			{
				break;
			}

			int64 NewCommitted = (int64)Align(RequiredSize, SyncAllocCommitGranularity);
			NewCommitted = FMath::Min(NewCommitted, (int64)SyncAllocReservedSize);

			// Commit the pages BEFORE advancing the high-water mark.
			// VirtualAlloc(MEM_COMMIT) is idempotent on already-committed pages,
			// so overlapping commits from concurrent threads are safe.
			void* Result = VirtualAlloc(SyncAllocBase + OldCommitted, (SIZE_T)(NewCommitted - OldCommitted), MEM_COMMIT, PAGE_READWRITE);
			checkf(Result != nullptr, TEXT("SyncObjectAllocator: Failed to commit %llu bytes"), NewCommitted - OldCommitted);

			// Advance the high-water mark. If another thread already advanced past us, that's fine —
			// the pages are committed either way. We loop and re-check.
			FPlatformAtomics::InterlockedCompareExchange(&SyncAllocCommittedSize, NewCommitted, OldCommitted);
		}
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES void InitSyncObjectAllocator(SIZE_T ReservedSize, SIZE_T CommitGranularity)
	{
		SyncAllocReservedSize = ReservedSize;
		SyncAllocCommitGranularity = CommitGranularity;
		SyncAllocBase = (uint8*)VirtualAlloc(NULL, ReservedSize, MEM_RESERVE, PAGE_NOACCESS);
		checkf(SyncAllocBase != nullptr, TEXT("SyncObjectAllocator: Failed to reserve %llu bytes of virtual address space"), ReservedSize);
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES void ShutdownSyncObjectAllocator()
	{
		if (SyncAllocBase)
		{
			VirtualFree(SyncAllocBase, 0, MEM_RELEASE);
			SyncAllocBase = nullptr;
			SyncAllocCommittedSize = 0;
			SyncAllocOffset = 0;
		}
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES void* SyncObjectAllocatorAlloc(SIZE_T Size)
	{
		constexpr SIZE_T CacheLineSize = 64;
		const SIZE_T AlignedSize = Align(Size, CacheLineSize);
		const int64 OldOffset = FPlatformAtomics::InterlockedAdd(&SyncAllocOffset, (int64)AlignedSize);

		checkf((SIZE_T)OldOffset + AlignedSize <= SyncAllocReservedSize, TEXT("SyncObjectAllocator: Exceeded reserved size of %llu bytes"), SyncAllocReservedSize);

		SyncAllocEnsureCommitted((SIZE_T)OldOffset + AlignedSize);

		return SyncAllocBase + OldOffset;
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES void SyncObjectAllocatorReset()
	{
		if (SyncAllocCommittedSize > 0)
		{
			VirtualFree(SyncAllocBase, (SIZE_T)SyncAllocCommittedSize, MEM_DECOMMIT);
		}
		SyncAllocOffset = 0;
		SyncAllocCommittedSize = 0;
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES void SyncObjectAllocatorAddRef()
	{
		FPlatformAtomics::InterlockedIncrement(&SyncAllocGlobalRefCount);
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES void SyncObjectAllocatorRelease()
	{
		FPlatformAtomics::InterlockedDecrement(&SyncAllocGlobalRefCount);
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES int32 SyncObjectAllocatorGetRefCount()
	{
		return SyncAllocGlobalRefCount;
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES SIZE_T SyncObjectAllocatorGetCommittedSize()
	{
		return (SIZE_T)SyncAllocCommittedSize;
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES void InitShadowMemory()
	{
		SYSTEM_INFO si;
		GetSystemInfo(&si);

		PageSize = si.dwPageSize;
		PageSizeBitShift = FPlatformMath::CeilLogTwo(PageSize);
		PageSizeMask = ~((1 << PageSizeBitShift) - 1);

		// -----------
		// Low Application Address Space
		// -----------
		// Shadow
		// -----------
		// High Application Address Space
		// -----------
		UPTRINT TotalAddressSpace = (UPTRINT)si.lpMaximumApplicationAddress - (UPTRINT)si.lpMinimumApplicationAddress;

		// We need 64 bytes of shadow for each 8 bytes of real application data. 8:1
		// We need a pointer to a clock bank (8 bytes) for each 8 bytes of application data 1:1
		// Reserve 9/10 of the space for the shadow so that we leave the application with 1/10 of the total address space
		ShadowSize = Align(9 * (TotalAddressSpace / 10), PageSize);

		// The 8:1 space so that we can compute the clock bank base address
		UPTRINT ShadowAccessSize = Align(8 * (TotalAddressSpace / 10), PageSize);

		// Because page faults are super slow when attached with a debugger
		// we're going to use a bitmap view where each 4KB page of the shadow bitmap
		// is a bit. When that bit is 1, the page has already been committed so we
		// don't need to call VirtualAlloc for that page.
		// The required memory for this scheme is a maximum of 4GB to support 128TB of address space.
		ShadowBitmapSize = Align((ShadowSize / PageSize) >> 3, PageSize);

		// Because zeroing 4GB is very slow when unmapping shadow.
		// We keep another 128KB of bits to know which page in the ShadowBitmap have been committed
		// so we just have to zero those page instead of uncommitting the whole 4GB and recommitting it.
		DirtyShadowBitmapSize = Align((ShadowBitmapSize / PageSize) >> 3, PageSize);

		// Let VirtualAlloc decide the best region to reserve. This works around a Windows 10 bug where specifying 
		// a base address when reserving large regions can cause extreme system-wide performance degradation.
		ShadowBitmapBase = (UPTRINT)VirtualAlloc(nullptr, ShadowSize + ShadowBitmapSize + DirtyShadowBitmapSize, MEM_RESERVE, PAGE_READWRITE);
		if (!ShadowBitmapBase)
		{
			UE_LOGF(LogRaceDetector, Fatal, "Failed to reserve shadow memory (err: %d)", GetLastError());
		}

		if (!VirtualAlloc((LPVOID)ShadowBitmapBase, ShadowBitmapSize + DirtyShadowBitmapSize, MEM_COMMIT, PAGE_READWRITE))
		{
			UE_LOGF(LogRaceDetector, Fatal, "Failed to commit shadow memory bitmap (err: %d)", GetLastError());
		}

		ShadowBitmapEnd = ShadowBitmapBase + ShadowBitmapSize;
		DirtyShadowBitmapBase = ShadowBitmapEnd;
		DirtyShadowBitmapEnd = DirtyShadowBitmapBase + DirtyShadowBitmapSize;

		// Put the real shadow memory after the bitmaps.
		ShadowBase = DirtyShadowBitmapEnd;
		ShadowEnd = ShadowBase + ShadowSize;
		ShadowClockBase = ShadowBase + ShadowAccessSize;
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES void DirtyShadowBitmapPage(UPTRINT ShadowBitmapAddress)
	{
		UPTRINT BitmapPage = (ShadowBitmapAddress - ShadowBitmapBase) >> PageSizeBitShift;

		uint8* DirtyBitmapPtr = (uint8*)(DirtyShadowBitmapBase + (BitmapPage >> 3));
		uint8 PageBit = (uint8)(1 << (BitmapPage & 7));
		checkSlow((UPTRINT)DirtyBitmapPtr < DirtyShadowBitmapEnd && (UPTRINT)DirtyBitmapPtr >= DirtyShadowBitmapBase);
		if ((*DirtyBitmapPtr & PageBit) == 0)
		{
			FPlatformAtomics::InterlockedOr((volatile int8*)DirtyBitmapPtr, (int8)PageBit);
		}
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES void ResetShadowBitmap()
	{
		// Scan all dirty bits in the 128KB range so that we know exactly which part of the 4GB shadow bitmap 
		// we need to zero out. This is actually a lot faster than uncommitting and recommitting the entire
		// range by the OS.
		constexpr int8 BitCount = sizeof(UPTRINT) << 3;
		for (UPTRINT DirtyPtr = DirtyShadowBitmapBase; DirtyPtr < DirtyShadowBitmapEnd; DirtyPtr += sizeof(UPTRINT))
		{
			UPTRINT DirtyBits = *(UPTRINT*)DirtyPtr;
			if (DirtyBits)
			{
				UPTRINT DirtyPage = ((DirtyPtr - DirtyShadowBitmapBase) << 3) << PageSizeBitShift;

				do
				{
					if (DirtyBits & 1)
					{
						UPTRINT ShadowBitmapAddr = ShadowBitmapBase + DirtyPage;
						memset((void*)ShadowBitmapAddr, 0, PageSize);
					}

					DirtyBits >>= 1;
					DirtyPage += PageSize;
				} while (DirtyBits);

				*(UPTRINT*)DirtyPtr = 0;
			}
		}
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES UPTRINT GetPageSize()
	{
		return PageSize;
	}

	template <typename T>
	FORCEINLINE INSTRUMENTATION_FUNCTION_ATTRIBUTES constexpr T AlignDown(T Val, uint64 Alignment)
	{
		static_assert(TIsIntegral<T>::Value || TIsPointer<T>::Value, "AlignDown expects an integer or pointer type");

		return (T)(((uint64)Val) & ~(Alignment - 1));
	}

	template <typename T>
	FORCEINLINE INSTRUMENTATION_FUNCTION_ATTRIBUTES constexpr T Align(T Val, uint64 Alignment)
	{
		static_assert(TIsIntegral<T>::Value || TIsPointer<T>::Value, "Align expects an integer or pointer type");

		return (T)(((uint64)Val + Alignment - 1) & ~(Alignment - 1));
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES bool IsShadowMemoryMapped(UPTRINT Base, UPTRINT Size)
	{
		UPTRINT AlignedStart = AlignDown(Base, PageSize);
		UPTRINT AlignedEnd   = Align(Base + Size, PageSize);

		UPTRINT ShadowPageStart = ((AlignedStart - ShadowBase) >> PageSizeBitShift);
		UPTRINT ShadowPageEnd = ((AlignedEnd - ShadowBase) >> PageSizeBitShift);

		for (UPTRINT ShadowPage = ShadowPageStart; ShadowPage < ShadowPageEnd; ++ShadowPage)
		{
			uint8* BitmapPtr = (uint8*)(ShadowBitmapBase + (ShadowPage >> 3));
			uint8 PageBit = (uint8)(1 << (ShadowPage & 7));
			if ((*BitmapPtr & PageBit))
			{
				return true;
			}
		}

		return false;
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES const TCHAR* GetCommandLine()
	{
		return ::GetCommandLineW();
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES uint16 CaptureStackBackTrace(uint32 FrameToSkip, uint32 FrameToCapture, void** Backtrace)
	{
		return ::RtlCaptureStackBackTrace(FrameToSkip, FrameToCapture, Backtrace, 0);
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES void ThrottleMemoryAccessesDuringShadowReset()
	{
		if (bIsResettingShadow && !bIsResettingShadowThread)
		{
			// If we're resetting the shadow memory, we delay resuming threads until the reset is done.
			// because we don't want to mess with the bitmap while it's being cleaned up.
			while (bIsResettingShadow)
			{
				Sleep(0);
			}
		}
	}

	// This is how shadow memory typically works, you just commit the page when there's a page fault.
	// But under the debugger, the kernel will send events to the debugger first and all these
	// events are taking a lot of time to process...Waaaayyy too much time. 
	// So we only rely on this exception handler when the debugger is not attached or during shadow resets.
	INSTRUMENTATION_FUNCTION_ATTRIBUTES static LONG CALLBACK ShadowExceptionHandler(PEXCEPTION_POINTERS ExceptionPointers)
	{
		if (ExceptionPointers->ExceptionRecord->ExceptionCode == EXCEPTION_ACCESS_VIOLATION &&
			ExceptionPointers->ExceptionRecord->NumberParameters >= 2)
		{
			const uintptr_t Address = (uintptr_t)ExceptionPointers->ExceptionRecord->ExceptionInformation[1];

			// Only handle exception inside the shadow memory address space.
			if (Address >= ShadowBase && Address < ShadowBase + ShadowSize)
			{
				ThrottleMemoryAccessesDuringShadowReset();

				uintptr_t BaseAddress = AlignDown(Address, PageSize);
				uintptr_t Result = (uintptr_t)::VirtualAlloc((LPVOID)BaseAddress, PageSize, MEM_COMMIT, PAGE_READWRITE);
				if (Result == BaseAddress)
				{
					bHasShadowMemoryMapped = true;

					// Mark the page as committed in the bitmap. We use the bitmap when the debugger is
					// present and to track memory usage of the shadow.
					UPTRINT ShadowPage = (BaseAddress - ShadowBase) >> PageSizeBitShift;

					uint8* BitmapPtr = (uint8*)(ShadowBitmapBase + (ShadowPage >> 3));
					uint8 PageBit = (uint8)(1 << (ShadowPage & 7));
					
					FPlatformAtomics::InterlockedOr((volatile int8*)BitmapPtr, (int8)PageBit);

					DirtyShadowBitmapPage((UPTRINT)BitmapPtr);

					return EXCEPTION_CONTINUE_EXECUTION;
				}
			}
		}

		return EXCEPTION_CONTINUE_SEARCH;
	}

	// Commit shadow memory so that we can read and write to it.
	INSTRUMENTATION_FUNCTION_ATTRIBUTES void MapShadowMemory(UPTRINT Base, UPTRINT Size)
	{
		UPTRINT AlignedStart = AlignDown(Base, PageSize);
		UPTRINT AlignedEnd = Align(Base + Size, PageSize);

		UPTRINT ShadowPageStart = ((AlignedStart - ShadowBase) >> PageSizeBitShift);
		UPTRINT ShadowPageEnd = ((AlignedEnd - ShadowBase) >> PageSizeBitShift);

		// Do not apply any modifications on the bitmap before allocating memory
		// since other threads might also want the same memory.
		bool bNeedMapping = false;
		for (UPTRINT ShadowPage = ShadowPageStart; ShadowPage < ShadowPageEnd; ++ShadowPage)
		{
			uint8* BitmapPtr = (uint8*)(ShadowBitmapBase + (ShadowPage >> 3));
			uint8 PageBit = (uint8)(1 << (ShadowPage & 7));
			if ((*BitmapPtr & PageBit) == 0)
			{
				// We found at least one uncommitted region, bail out.
				bNeedMapping = true;
				break;
			}
		}

		if (bNeedMapping)
		{
			ThrottleMemoryAccessesDuringShadowReset();

			VirtualAlloc((void*)AlignedStart, AlignedEnd - AlignedStart, MEM_COMMIT, PAGE_READWRITE);

			bHasShadowMemoryMapped = true;

			// Mark all the committed page in the bitmap in a thread-safe manner.
			// We could unroll this to make it faster but this is not on the critical path anyway.
			for (UPTRINT ShadowPage = ShadowPageStart; ShadowPage < ShadowPageEnd; ShadowPage++)
			{
				uint8* BitmapPtr = (uint8*)(ShadowBitmapBase + (ShadowPage >> 3));
				uint8 PageBit = (uint8)(1 << (ShadowPage & 7));
				if ((*BitmapPtr & PageBit) == 0)
				{
					FPlatformAtomics::InterlockedOr((volatile int8*)BitmapPtr, (int8)PageBit);

					DirtyShadowBitmapPage((UPTRINT)BitmapPtr);
				}
			}
		}
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES void UnmapShadowMemory()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UE::Sanitizer::UnmapShadowMemory);

		// This gets set during mapping and mapping can happen even while we unmap
		// so we need to reset the flag first.
		bHasShadowMemoryMapped = false;
		// Decommit the shadow so it goes back to zeros
		VirtualFree((LPVOID*)ShadowBase, ShadowSize, MEM_DECOMMIT);
		ResetShadowBitmap();
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES UPTRINT GetShadowMemoryBase()
	{
		return ShadowBase;
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES UPTRINT GetShadowMemorySize()
	{
		return ShadowSize;
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES UPTRINT GetShadowClockBase()
	{
		return ShadowClockBase;
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES uint64 GetShadowMemoryUsage()
	{
		uint64 ShadowPageCount = 0;

		// Scan all dirty bits in the 128KB range so that we know exactly which part of the 4GB shadow bitmap 
		// we need to access.
		constexpr int8 BitCount = sizeof(UPTRINT) << 3;
		for (UPTRINT DirtyPtr = DirtyShadowBitmapBase; DirtyPtr < DirtyShadowBitmapEnd; DirtyPtr += sizeof(UPTRINT))
		{
			UPTRINT DirtyBits = *(UPTRINT*)DirtyPtr;
			if (DirtyBits)
			{
				UPTRINT DirtyPage = ((DirtyPtr - DirtyShadowBitmapBase) << 3) << PageSizeBitShift;

				do
				{
					if (DirtyBits & 1)
					{
						UPTRINT ShadowBitmapAddr = ShadowBitmapBase + DirtyPage;

						// Scan the whole 4KB of bits to know how many bits are used
						// which tells us how many shadow pages are committed.
						for (UPTRINT Addr = 0; Addr < PageSize; Addr += 8)
						{
							uint64* Bits = (uint64*)(ShadowBitmapAddr + Addr);

							ShadowPageCount += FMath::CountBits64(*Bits);
						}
					}

					DirtyBits >>= 1;
					DirtyPage += PageSize;
				} while (DirtyBits);
			}
		}

		return ShadowPageCount << PageSizeBitShift;
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES bool InitializePlatform()
	{
		AddVectoredExceptionHandler(TRUE, &ShadowExceptionHandler);
		
		HideFirstChanceExceptionInVisualStudio();

		PopulateDetouredFunctions();

		SanitizerThreadHandle = CreateThread(
			nullptr,
			0,
			SanitizerThreadProc,
			0,
			0,
			0
		);

		DetourTransactionBegin();
		DetourUpdateThread(GetCurrentThread());

		for (const auto [RealFn, DetouredFn] : DetouredFunctions)
		{
			DetourAttach(RealFn, DetouredFn);
		}

		return DetourTransactionCommit() == NO_ERROR;
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES bool CleanupPlatform()
	{
		bSanitizerThreadContinue = false;
		WaitForSingleObject(SanitizerThreadHandle, INFINITE);
		CloseHandle(SanitizerThreadHandle);

		DetourTransactionBegin();
		DetourUpdateThread(GetCurrentThread());

		for (const auto [RealFn, DetouredFn] : DetouredFunctions)
		{
			DetourDetach(RealFn, DetouredFn);
		}

		return DetourTransactionCommit() == NO_ERROR;
	}
}

// When building in non monolithic we need to hook ourselves
// as fast as possible so do it here.
INSTRUMENTATION_FUNCTION_ATTRIBUTES BOOL WINAPI DllMain(
	HINSTANCE HinstDLL,
	DWORD     dwReason,
	LPVOID    LpvReserved
)
{
	switch (dwReason)
	{
	case DLL_PROCESS_ATTACH:
		UE::Sanitizer::RaceDetector::Initialize();
		break;
	case DLL_PROCESS_DETACH:
		UE::Sanitizer::RaceDetector::Shutdown();
		break;
	}

	return TRUE;
}

typedef NTSTATUS(NTAPI DBGUIREMOTEBREAKIN)
(
	IN HANDLE hProcess
);

NTSTATUS(NTAPI* TrueDbgUiRemoteBreakin)(HANDLE hProcess);
INSTRUMENTATION_FUNCTION_ATTRIBUTES NTSTATUS NTAPI DetouredDbgUiRemoteBreakin(HANDLE hProcess)
{
        // Notify visual studio as soon as it attaches itself to avoid seeing unhandled exceptions
	// for late attachments.
	Platform::HideFirstChanceExceptionInVisualStudio();

	return TrueDbgUiRemoteBreakin(hProcess);
}

INSTRUMENTATION_FUNCTION_ATTRIBUTES int DetourBootFunctions()
{
	DetourTransactionBegin();
	DetourUpdateThread(GetCurrentThread());

	TrueCreateThread = ::CreateThread;
	DetourAttach(&TrueCreateThread, DetouredCreateThread);
	True_beginthreadex = ::_beginthreadex;
	DetourAttach(&True_beginthreadex, Detoured_beginthreadex);
	TrueExitThread = ::ExitThread;
	DetourAttach(&TrueExitThread, DetouredExitThread);
	TrueDbgUiRemoteBreakin = (DBGUIREMOTEBREAKIN*)GetProcAddress(GetModuleHandleA("ntdll"), "DbgUiRemoteBreakin");
	DetourAttach(&TrueDbgUiRemoteBreakin, DetouredDbgUiRemoteBreakin);
	TrueFreeLibraryAndExitThread = ::FreeLibraryAndExitThread;
	DetourAttach(&TrueFreeLibraryAndExitThread, DetouredFreeLibraryAndExitThread);
	True_Cnd_do_broadcast_at_thread_exit = ::_Cnd_do_broadcast_at_thread_exit;
	DetourAttach(&True_Cnd_do_broadcast_at_thread_exit, Detoured_Cnd_do_broadcast_at_thread_exit);

	LONG Result = DetourTransactionCommit();
	if (Result != NO_ERROR)
	{
		UE_LOGF(LogRaceDetector, Fatal, "Could not install detoured boot functions (error %d)", Result);
		return Result;
	}

	return 0;
}

extern "C" INSTRUMENTATION_FUNCTION_ATTRIBUTES int PreInit()
{
	InitializeBoot();
	return DetourBootFunctions();
}

#pragma section(".CRT$XCT",long,read)
__declspec(allocate(".CRT$XCT")) int (*SanitizerPreStaticInitFn)() = PreInit;

#include "Windows/HideWindowsPlatformTypes.h"

#endif // PLATFORM_WINDOWS && USING_INSTRUMENTATION
