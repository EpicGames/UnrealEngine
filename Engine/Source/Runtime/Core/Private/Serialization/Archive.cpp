// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/Archive.h"
#include "Serialization/ArchiveProxy.h"
#include "Math/UnrealMathUtility.h"
#include "HAL/UnrealMemory.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "UObject/NameTypes.h"
#include "Logging/LogMacros.h"
#include "Logging/StructuredLog.h"
#include "Misc/Parse.h"
#include "UObject/ObjectVersion.h"
#include "Serialization/NameAsStringProxyArchive.h"
#include "Misc/CommandLine.h"
#include "Internationalization/Text.h"
#include "Stats/StatsMisc.h"
#include "Stats/Stats.h"
#include "Async/AsyncWork.h"
#include "Serialization/CustomVersion.h"
#include "Misc/EngineVersion.h"
#include "Misc/NetworkVersion.h"
#include "Interfaces/ITargetPlatform.h"
#include "Serialization/CompressedChunkInfo.h"
#include "Serialization/ArchiveSerializedPropertyChain.h"
#include "UObject/NameTypes.h"
#include "Compression/CompressionUtil.h"
#include "UObject/UnrealNames.h"
#include "Misc/EngineNetworkCustomVersion.h"

PRAGMA_DISABLE_UNSAFE_TYPECAST_WARNINGS

namespace ArchiveUtil
{

template<typename T>
FArchive& SerializeByteOrderSwapped(FArchive& Ar, T& Value)
{
	static_assert(!TIsSigned<T>::Value, "To reduce the number of template instances, cast 'Value' to a uint16&, uint32& or uint64& prior to the call.");

	if (Ar.IsLoading())
	{
		// Read and swap.
		Ar.Serialize(&Value, sizeof(T));
		Value = ByteSwap(Value);
	}
	else // Saving
	{
		// Swap and write.
		T SwappedValue = ByteSwap(Value);
		Ar.Serialize(&SwappedValue, sizeof(T));
	}
	return Ar;
}

} // namespace ArchiveUtil

#if UE_INTERNAL_LINKER_BOUNDS
FArchiveBoundsScope::FArchiveBoundsScope(FArchive* InOwner, FArchiveBounds InOriginalBounds)
	: Owner(InOwner)
	, OriginalBounds(InOriginalBounds)
{
	check(Owner);
}

FArchiveBoundsScope::~FArchiveBoundsScope()
{
	check(Owner);
	Owner->SetBoundsImpl(OriginalBounds);
}

FArchiveBoundsScope FArchive::SetBounds(int64_t Size)
{
	FArchiveBounds PrevBounds = GetBoundsImpl();
	SetBoundsImpl({ Tell(), Size});
	return FArchiveBoundsScope(this, PrevBounds);
}

void FArchive::SetBoundsImpl(FArchiveBounds NewBounds)
{
}

FArchiveBounds FArchive::GetBoundsImpl() const
{
	return {};
}
#endif // UE_INTERNAL_LINKER_BOUNDS

/*-----------------------------------------------------------------------------
	FArchive implementation.
-----------------------------------------------------------------------------*/

FArchiveState::FArchiveState()
{
#if DEVIRTUALIZE_FLinkerLoad_Serialize
	ActiveFPLB = &InlineFPLB;
#endif
	SerializedPropertyChain = nullptr;

#if USE_STABLE_LOCALIZATION_KEYS
	LocalizationNamespacePtr = nullptr;
#endif // USE_STABLE_LOCALIZATION_KEYS

	Reset();
}

FArchiveState::FArchiveState(const FArchiveState& ArchiveToCopy)
{
#if DEVIRTUALIZE_FLinkerLoad_Serialize
	ActiveFPLB = &InlineFPLB;
#endif
#if USE_STABLE_LOCALIZATION_KEYS
	LocalizationNamespacePtr = nullptr;
#endif // USE_STABLE_LOCALIZATION_KEYS

	CopyTrivialFArchiveStatusMembers(ArchiveToCopy);

	SerializedPropertyChain = nullptr;
	SetSerializedPropertyChain(ArchiveToCopy.SerializedPropertyChain, ArchiveToCopy.SerializedProperty);

	// Don't know why this is set to false, but this is what the original copying code did
	ArIsFilterEditorOnly  = false;

	bCustomVersionsAreReset = ArchiveToCopy.bCustomVersionsAreReset;
	if (ArchiveToCopy.CustomVersionContainer)
	{
		CustomVersionContainer = new FCustomVersionContainer(*ArchiveToCopy.CustomVersionContainer);
	}
	else
	{
		CustomVersionContainer = nullptr;
	}
}

FArchiveState& FArchiveState::operator=(const FArchiveState& ArchiveToCopy)
{
#if DEVIRTUALIZE_FLinkerLoad_Serialize
	ActiveFPLB = &InlineFPLB;
	ActiveFPLB->Reset();
#endif
	CopyTrivialFArchiveStatusMembers(ArchiveToCopy);

	SetSerializedPropertyChain(ArchiveToCopy.SerializedPropertyChain, ArchiveToCopy.SerializedProperty);

	// Don't know why this is set to false, but this is what the original copying code did
	ArIsFilterEditorOnly  = false;

	bCustomVersionsAreReset = ArchiveToCopy.bCustomVersionsAreReset;
	if (ArchiveToCopy.CustomVersionContainer)
	{
		if (!CustomVersionContainer)
		{
			CustomVersionContainer = new FCustomVersionContainer(*ArchiveToCopy.CustomVersionContainer);
		}
		else
		{
			*CustomVersionContainer = *ArchiveToCopy.CustomVersionContainer;
		}
	}
	else if (CustomVersionContainer)
	{
		delete CustomVersionContainer;
		CustomVersionContainer = nullptr;
	}

	return *this;
}

FArchiveState::~FArchiveState()
{
	checkf(NextProxy == nullptr, TEXT("Archive destroyed before its proxies"));

	delete CustomVersionContainer;

	delete SerializedPropertyChain;

#if USE_STABLE_LOCALIZATION_KEYS
	delete LocalizationNamespacePtr;
#endif // USE_STABLE_LOCALIZATION_KEYS
}

void FArchiveState::Reset()
{
#if DEVIRTUALIZE_FLinkerLoad_Serialize
	ActiveFPLB->Reset();
#endif
	ArUEVer								= GPackageFileUEVersion;
	ArLicenseeUEVer						= GPackageFileLicenseeUEVersion;
	ArEngineVer							= FEngineVersion::Current();
	ArIsLoading							= false;
	ArIsLoadingFromCookedPackage		= false;
	ArIsSaving							= false;
	ArIsTransacting						= false;
	ArIsTextFormat						= false;
	ArWantBinaryPropertySerialization	= false;
	ArUseUnversionedPropertySerialization = false;
	ArForceUnicode						= false;
	ArIsPersistent						= false;
	ArIsError							= false;
	ArIsCriticalError					= false;
	ArContainsCode						= false;
	ArContainsMap						= false;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	ArRequiresLocalizationGather		= false;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	ArForceByteSwapping					= false;
	ArSerializingDefaults				= false;
	ArIgnoreArchetypeRef				= false;
	ArNoDelta							= false;
	ArNoIntraPropertyDelta				= false;
	ArIgnoreOuterRef					= false;
	ArIgnoreClassGeneratedByRef			= false;
	ArIgnoreClassRef					= false;
	ArAllowLazyLoading					= false;
	ArIsObjectReferenceCollector		= false;
	ArIsModifyingWeakAndStrongReferences= false;
	ArIsCountingMemory					= false;
	ArPortFlags							= 0;
	ArShouldSkipBulkData				= false;
	ArShouldSkipCompilingAssets			= false;
	ArMaxSerializeSize					= 0;
	ArIsFilterEditorOnly				= false;
	ArIsSavingOptionalObject			= false;
	ArIsSaveGame						= false;
	ArIsNetArchive						= false;
	ArCustomPropertyList				= nullptr;
	ArUseCustomPropertyList				= false;
	ArMergeOverrides					= false;
	ArPreserveArrayElements				= false;
	ArShouldSkipUpdateCustomVersion		= false;
#if UE_WITH_REMOTE_OBJECT_HANDLE
	ArIsMigratingRemoteObjects			= false;
#endif
	SavePackageData						= nullptr;
	SerializedProperty					= nullptr;

	delete SerializedPropertyChain;
	SerializedPropertyChain = nullptr;

#if USE_STABLE_LOCALIZATION_KEYS
	SetBaseLocalizationNamespace(FString());
#endif // USE_STABLE_LOCALIZATION_KEYS

#if WITH_EDITOR
	ArDebugSerializationFlags			= 0;
#endif

	// Reset all custom versions to the current registered versions.
	ResetCustomVersions();
}

void FArchiveState::CopyTrivialFArchiveStatusMembers(const FArchiveState& ArchiveToCopy)
{
	ArUEVer                              = ArchiveToCopy.ArUEVer;
	ArLicenseeUEVer                      = ArchiveToCopy.ArLicenseeUEVer;
	ArEngineVer                          = ArchiveToCopy.ArEngineVer;
	ArIsLoading                          = ArchiveToCopy.ArIsLoading;
	ArIsLoadingFromCookedPackage         = ArchiveToCopy.ArIsLoadingFromCookedPackage;
	ArIsSaving                           = ArchiveToCopy.ArIsSaving;
	ArIsTransacting                      = ArchiveToCopy.ArIsTransacting;
	ArIsTextFormat                       = ArchiveToCopy.ArIsTextFormat;
	ArWantBinaryPropertySerialization    = ArchiveToCopy.ArWantBinaryPropertySerialization;
	ArUseUnversionedPropertySerialization = ArchiveToCopy.ArUseUnversionedPropertySerialization;
	ArForceUnicode                       = ArchiveToCopy.ArForceUnicode;
	ArIsPersistent                       = ArchiveToCopy.ArIsPersistent;
	ArIsError                            = ArchiveToCopy.ArIsError;
	ArIsCriticalError                    = ArchiveToCopy.ArIsCriticalError;
	ArContainsCode                       = ArchiveToCopy.ArContainsCode;
	ArContainsMap                        = ArchiveToCopy.ArContainsMap;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	ArRequiresLocalizationGather         = ArchiveToCopy.ArRequiresLocalizationGather;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	ArForceByteSwapping                  = ArchiveToCopy.ArForceByteSwapping;
	ArSerializingDefaults                = ArchiveToCopy.ArSerializingDefaults;
	ArIgnoreArchetypeRef                 = ArchiveToCopy.ArIgnoreArchetypeRef;
	ArNoDelta                            = ArchiveToCopy.ArNoDelta;
	ArNoIntraPropertyDelta               = ArchiveToCopy.ArNoIntraPropertyDelta;
	ArIgnoreOuterRef                     = ArchiveToCopy.ArIgnoreOuterRef;
	ArIgnoreClassGeneratedByRef          = ArchiveToCopy.ArIgnoreClassGeneratedByRef;
	ArIgnoreClassRef                     = ArchiveToCopy.ArIgnoreClassRef;
	ArAllowLazyLoading                   = ArchiveToCopy.ArAllowLazyLoading;
	ArIsObjectReferenceCollector         = ArchiveToCopy.ArIsObjectReferenceCollector;
	ArIsModifyingWeakAndStrongReferences = ArchiveToCopy.ArIsModifyingWeakAndStrongReferences;
	ArIsCountingMemory                   = ArchiveToCopy.ArIsCountingMemory;
	ArPortFlags                          = ArchiveToCopy.ArPortFlags;
	ArShouldSkipBulkData                 = ArchiveToCopy.ArShouldSkipBulkData;
	ArShouldSkipCompilingAssets          = ArchiveToCopy.ArShouldSkipCompilingAssets;
	ArMaxSerializeSize                   = ArchiveToCopy.ArMaxSerializeSize;
	ArIsFilterEditorOnly                 = ArchiveToCopy.ArIsFilterEditorOnly;
	ArIsSavingOptionalObject             = ArchiveToCopy.ArIsSavingOptionalObject;
	ArIsSaveGame                         = ArchiveToCopy.ArIsSaveGame;
	ArIsNetArchive                       = ArchiveToCopy.ArIsNetArchive;
	ArCustomPropertyList                 = ArchiveToCopy.ArCustomPropertyList;
	ArUseCustomPropertyList              = ArchiveToCopy.ArUseCustomPropertyList;
	ArMergeOverrides					 = ArchiveToCopy.ArMergeOverrides;
	ArPreserveArrayElements					 = ArchiveToCopy.ArPreserveArrayElements;
	ArShouldSkipUpdateCustomVersion		 = ArchiveToCopy.ArShouldSkipUpdateCustomVersion;
#if UE_WITH_REMOTE_OBJECT_HANDLE
	ArIsMigratingRemoteObjects			 = ArchiveToCopy.ArIsMigratingRemoteObjects;
#endif
	SavePackageData						 = ArchiveToCopy.SavePackageData;
	SerializedProperty					 = ArchiveToCopy.SerializedProperty;
#if USE_STABLE_LOCALIZATION_KEYS
	SetBaseLocalizationNamespace(ArchiveToCopy.GetBaseLocalizationNamespace());
#endif // USE_STABLE_LOCALIZATION_KEYS
}


void FArchiveState::LinkProxy(FArchiveState& Inner, FArchiveState& Proxy)
{
	Proxy.NextProxy = Inner.NextProxy;
	Inner.NextProxy = &Proxy;
}

void FArchiveState::UnlinkProxy(FArchiveState& Inner, FArchiveState& Proxy)
{
	FArchiveState* Prev = &Inner;
	while (Prev->NextProxy != &Proxy)
	{
		Prev = Prev->NextProxy;
		checkf(Prev, TEXT("Proxy link not found - likely  lifetime violation"));
	}

	Prev->NextProxy = Proxy.NextProxy;
	Proxy.NextProxy = nullptr;
}

template<typename T>
FORCEINLINE void FArchiveState::ForEachState(T Func)
{
	FArchiveState& RootState = GetInnermostState();
	Func(RootState);

	for (FArchiveState* Proxy = RootState.NextProxy; Proxy; Proxy = Proxy->NextProxy)
	{
		Func(*Proxy);
	}
}

void FArchiveState::SetArchiveState(const FArchiveState& InState)
{
	ForEachState([&InState](FArchiveState& State) { State = InState; });
}

void FArchiveState::SetError()
{
	ForEachState([](FArchiveState& State) { State.ArIsError = true; });
}

void FArchiveState::SetCriticalError()
{
	ForEachState([](FArchiveState& State) { State.ArIsError = State.ArIsCriticalError = true; });
}

#if UE_INTERNAL_LINKER_BOUNDS
void FArchiveState::ClearCriticalError()
{
	ForEachState([](FArchiveState& State) { State.ArIsError = State.ArIsCriticalError = false; });
}
#endif // UE_INTERNAL_LINKER_BOUNDS

void FArchiveState::ClearError()
{
	ForEachState([](FArchiveState& State) { State.ArIsError = false; });
}

/**
 * Returns the name of the Archive.  Useful for getting the name of the package a struct or object
 * is in when a loading error occurs.
 *
 * This is overridden for the specific Archive Types
 **/
FString FArchiveState::GetArchiveName() const
{
	return TEXT("FArchive");
}

void FArchiveState::GetSerializedPropertyChain(TArray<class FProperty*>& OutProperties) const
{
	if (SerializedPropertyChain)
	{
		const int32 NumProperties = SerializedPropertyChain->GetNumProperties();
		OutProperties.Reserve(NumProperties);

		for (int32 PropertyIndex = 0; PropertyIndex < NumProperties; ++PropertyIndex)
		{
			OutProperties.Add(SerializedPropertyChain->GetPropertyFromStack(PropertyIndex));
		}
	}
}

void FArchiveState::SetSerializedPropertyChain(const FArchiveSerializedPropertyChain* InSerializedPropertyChain, class FProperty* InSerializedPropertyOverride)
{
	if (InSerializedPropertyChain && InSerializedPropertyChain->GetNumProperties() > 0)
	{
		if (!SerializedPropertyChain)
		{
			SerializedPropertyChain = new FArchiveSerializedPropertyChain();
		}
		*SerializedPropertyChain = *InSerializedPropertyChain;
	}
	else
	{
		delete SerializedPropertyChain;
		SerializedPropertyChain = nullptr;
	}

	if (InSerializedPropertyOverride)
	{
		SerializedProperty = InSerializedPropertyOverride;
	}
	else if (SerializedPropertyChain && SerializedPropertyChain->GetNumProperties() > 0)
	{
		SerializedProperty = SerializedPropertyChain->GetPropertyFromStack(0);
	}
	else
	{
		SerializedProperty = nullptr;
	}
}

void FArchive::PushSerializedProperty(class FProperty* InProperty, const bool bIsEditorOnlyProperty)
{
	if (InProperty)
	{
		// Push this property into the chain
		if (!SerializedPropertyChain)
		{
			SerializedPropertyChain = new FArchiveSerializedPropertyChain();
		}
		SerializedPropertyChain->PushProperty(InProperty, bIsEditorOnlyProperty);

		// Update the serialized property pointer with the new head
		SerializedProperty = InProperty;
	}
}

void FArchive::PopSerializedProperty(class FProperty* InProperty, const bool bIsEditorOnlyProperty)
{
	if (InProperty)
	{
		// Pop this property from the chain
		check(SerializedPropertyChain);
		SerializedPropertyChain->PopProperty(InProperty, bIsEditorOnlyProperty);

		// Update the serialized property pointer with the new head
		if (SerializedPropertyChain->GetNumProperties() > 0)
		{
			SerializedProperty = SerializedPropertyChain->GetPropertyFromStack(0);
		}
		else
		{
			SerializedProperty = nullptr;
		}
	}
}

#if WITH_EDITORONLY_DATA
bool FArchiveState::IsEditorOnlyPropertyOnTheStack() const
{
	return SerializedPropertyChain && SerializedPropertyChain->HasEditorOnlyProperty();
}
#endif

#if USE_STABLE_LOCALIZATION_KEYS
void FArchiveState::SetBaseLocalizationNamespace(const FString& InLocalizationNamespace)
{
	if (InLocalizationNamespace.IsEmpty())
	{
		delete LocalizationNamespacePtr;
		LocalizationNamespacePtr = nullptr;
	}
	else
	{
		if (!LocalizationNamespacePtr)
		{
			LocalizationNamespacePtr = new FString();
		}
		*LocalizationNamespacePtr = InLocalizationNamespace;
	}
}
FString FArchiveState::GetBaseLocalizationNamespace() const
{
	return LocalizationNamespacePtr ? *LocalizationNamespacePtr : FString();
}
void FArchiveState::SetLocalizationNamespace(const FString& InLocalizationNamespace)
{
	SetBaseLocalizationNamespace(InLocalizationNamespace);
}
FString FArchiveState::GetLocalizationNamespace() const
{
	return GetBaseLocalizationNamespace();
}
#endif // USE_STABLE_LOCALIZATION_KEYS

#if WITH_EDITOR
FArchive::FScopeAddDebugData::FScopeAddDebugData(FArchive& InAr, const FName& DebugData) : Ar(InAr)
{
	Ar.PushDebugDataString(DebugData);
}

void FArchive::PushDebugDataString(const FName& DebugData)
{
}
#endif

FArchive& FArchive::operator<<( FText& Value )
{
	FText::SerializeText(*this, Value);
	return *this;
}

FArchive& FArchive::operator<<(struct FLazyObjectPtr& Value)
{
	// The base FArchive does not implement this method. Use FArchiveUObject instead.
	UE_LOGF(LogSerialization, Fatal, "FArchive does not support FLazyObjectPtr serialization. Use FArchiveUObject instead.");
	return *this;
}

FArchive& FArchive::operator<<(struct FObjectPtr& Value)
{
	// The base FArchive does not implement this method. Use FArchiveUObject instead.
	UE_LOGF(LogSerialization, Fatal, "FArchive does not support FObjectPtr serialization. Use FArchiveUObject instead.");
	return *this;
}

FArchive& FArchive::operator<<(struct FSoftObjectPtr& Value)
{
	// The base FArchive does not implement this method. Use FArchiveUObject instead.
	UE_LOGF(LogSerialization, Fatal, "FArchive does not support FSoftObjectPtr serialization. Use FArchiveUObject instead.");
	return *this;
}

FArchive& FArchive::operator<<(struct FSoftObjectPath& Value)
{
	// The base FArchive does not implement this method. Use FArchiveUObject instead.
	UE_LOGF(LogSerialization, Fatal, "FArchive does not support FSoftObjectPath serialization. Use FArchiveUObject instead.");
	return *this;
}

FArchive& FArchive::operator<<(struct FWeakObjectPtr& Value)
{
	// The base FArchive does not implement this method. Use FArchiveUObject instead.
	UE_LOGF(LogSerialization, Fatal, "FArchive does not support FWeakObjectPtr serialization. Use FArchiveUObject instead.");
	return *this;
}

#if WITH_EDITOR
void FArchive::SerializeBool( bool& D )
{
	// Serialize bool as if it were UBOOL (legacy, 32 bit int).
	uint32 OldUBoolValue;
#if DEVIRTUALIZE_FLinkerLoad_Serialize
	const uint8 * RESTRICT Src = this->ActiveFPLB->StartFastPathLoadBuffer;
	if (Src + sizeof(uint32) <= this->ActiveFPLB->EndFastPathLoadBuffer)
	{
		OldUBoolValue = FPlatformMemory::ReadUnaligned<uint32>(Src);
		this->ActiveFPLB->StartFastPathLoadBuffer += 4;
	}
	else
#endif
	{
		OldUBoolValue = D ? 1 : 0;
		this->Serialize(&OldUBoolValue, sizeof(OldUBoolValue));
	}

	if (OldUBoolValue > 1)
	{
		UE_LOGFMT(LogSerialization, Error, "Invalid boolean encountered while reading archive {ArchiveName} - stream is most likely corrupted.", GetArchiveName());

		this->SetError();
	}

	if (this->IsLoading())
	{
		// Only write to our input if we are loading to make sure we don't
		// write to 'read-only' memory (e.g. FMemoryWriter) as this can cause TSAN
		// validation races
		D = !!OldUBoolValue;
	}
}
#endif

const FCustomVersionContainer& FArchiveState::GetCustomVersions() const
{
	if (!CustomVersionContainer)
	{
		CustomVersionContainer = new FCustomVersionContainer;
	}

	if (bCustomVersionsAreReset)
	{
		bCustomVersionsAreReset = false;

		// If the archive is for reading then we want to use currently registered custom versions, otherwise we expect
		// serialization code to use UsingCustomVersion to populate the container.
		if (this->IsLoading())
		{
			*CustomVersionContainer = FCurrentCustomVersions::GetAll();
		}
		else
		{
			CustomVersionContainer->Empty();
		}
	}

	return *CustomVersionContainer;
}

void FArchiveState::SetCustomVersions(const FCustomVersionContainer& NewVersions)
{
	if (!CustomVersionContainer)
	{
		CustomVersionContainer = new FCustomVersionContainer(NewVersions);
	}
	else
	{
		*CustomVersionContainer = NewVersions;
	}
	bCustomVersionsAreReset = false;
}

void FArchiveState::ResetCustomVersions()
{
	bCustomVersionsAreReset = true;
}

void FArchive::UsingCustomVersion(const FGuid& Key)
{
	// If we're loading, we want to use the version that the archive was serialized with, not register a new one.
	if (IsLoading())
	{
		return;
	}

	ESetCustomVersionFlags SetVersionFlags = ArShouldSkipUpdateCustomVersion ? ESetCustomVersionFlags::SkipUpdateExistingVersion : ESetCustomVersionFlags::None;
	const_cast<FCustomVersionContainer&>(GetCustomVersions()).SetVersionUsingRegistry(Key, SetVersionFlags);
}

int32 FArchiveState::CustomVer(const FGuid& Key) const
{
	auto* CustomVersion = GetCustomVersions().GetVersion(Key);

	// If this fails, you have forgotten to make an Ar.UsingCustomVersion call
	// before serializing your custom version-dependent object.
	check(IsLoading() || CustomVersion);

	return CustomVersion ? CustomVersion->Version : -1;
}

void FArchiveState::SetShouldSkipUpdateCustomVersion(bool bShouldSkip)
{
	ForEachState([bShouldSkip](FArchiveState& State) { State.ArShouldSkipUpdateCustomVersion = bShouldSkip; });
}

void FArchiveState::SetMigratingRemoteObjects(bool bMigrating)
{
#if UE_WITH_REMOTE_OBJECT_HANDLE
	ForEachState([bMigrating](FArchiveState& State) { State.ArIsMigratingRemoteObjects = bMigrating; });
#else
	checkf(false, TEXT("SetMigratingRemoteObjects() can only be used in a build with remote object handles enabled"))
#endif
}

void FArchiveState::SetCustomVersion(const FGuid& Key, int32 Version, FName FriendlyName)
{
	const_cast<FCustomVersionContainer&>(GetCustomVersions()).SetVersion(Key, Version, FriendlyName);
}

FString FArchiveProxy::GetArchiveName() const
{
	return InnerArchive.GetArchiveName();
}

#if USE_STABLE_LOCALIZATION_KEYS
void FArchiveProxy::SetLocalizationNamespace(const FString& InLocalizationNamespace)
{
	InnerArchive.SetLocalizationNamespace(InLocalizationNamespace);
}
FString FArchiveProxy::GetLocalizationNamespace() const
{
	return InnerArchive.GetLocalizationNamespace();
}
#endif // USE_STABLE_LOCALIZATION_KEYS

FNameAsStringProxyArchive::~FNameAsStringProxyArchive() = default;

/**
 * Serialize the given FName as an FString
 */
FArchive& FNameAsStringProxyArchive::operator<<( class FName& N )
{
	if (IsLoading())
	{
		FString LoadedString;
		InnerArchive << LoadedString;
		N = FName(*LoadedString);
	}
	else
	{
		FString SavedString(N.ToString());
		InnerArchive << SavedString;
	}
	return *this;
}

/** Accumulative time spent in IsSaving portion of SerializeCompressed. */
CORE_API double GArchiveSerializedCompressedSavingTime = 0;



// MT compression disabled on console due to memory impact and lack of beneficial usage case.
#define WITH_MULTI_THREADED_COMPRESSION (WITH_EDITORONLY_DATA)
#if WITH_MULTI_THREADED_COMPRESSION
// Helper structure to keep information about async chunks that are in-flight.
class FAsyncCompressionChunk : public FNonAbandonableTask
{
public:
	/** Pointer to source (uncompressed) memory.								*/
	void* UncompressedBuffer;
	/** Pointer to destination (compressed) memory.								*/
	void* CompressedBuffer;
	/** Compressed size in bytes as passed to/ returned from compressor.		*/
	int32 CompressedSize;
	/** Uncompressed size in bytes as passed to compressor.						*/
	int32 UncompressedSize;
	/** Target platform for compressed data										*/
	int32 BitWindow;
	/** Format to compress with													*/
	FName CompressionFormat;
	/** Flags to control compression											*/
	ECompressionFlags Flags;

	/**
	 * Constructor, zeros everything
	 */
	FAsyncCompressionChunk()
		: UncompressedBuffer(0)
		, CompressedBuffer(0)
		, CompressedSize(0)
		, UncompressedSize(0)
		, BitWindow(DEFAULT_ZLIB_BIT_WINDOW)
		, CompressionFormat(NAME_None)
		, Flags(COMPRESS_NoFlags)
	{
	}
	/**
	 * Performs the async compression
	 */
	void DoWork()
	{
		// Compress from memory to memory.
		verify( FCompression::CompressMemory(CompressionFormat, CompressedBuffer, CompressedSize, UncompressedBuffer, UncompressedSize, Flags, BitWindow) );
	}

	FORCEINLINE TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FAsyncCompressionChunk, STATGROUP_ThreadPoolAsyncTasks);
	}
};
#endif		// WITH_MULTI_THREADED_COMPRESSION



void FArchive::SerializeCompressed(void* V, int64 Length, FName CompressionFormatCannotChange, ECompressionFlags Flags, bool bTreatBufferAsFileReader)
{
	// with this legacy/deprecated API you can NOT change the CompressionFormat and still load old files
	//  CompressionFormat must match exactly the format that was written to the file

	SerializeCompressedNew(V,Length,CompressionFormatCannotChange,CompressionFormatCannotChange,Flags,bTreatBufferAsFileReader,nullptr);
}

void FArchive::SerializeCompressedNew(void* V, int64 Length)
{
	SerializeCompressedNew(V,Length,NAME_Oodle,NAME_Zlib);
}

void FArchive::SerializeCompressedNew(void* V, int64 Length, FName CompressionFormatToEncode, FName CompressionFormatToDecodeOldV1Files,  ECompressionFlags Flags, bool bTreatBufferAsFileReader, int64 * OutPartialReadLength)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FArchive::SerializeCompressed);

	// CompressionFormatToEncode can be changed freely without breaking loading of old files
	// CompressionFormatToDecodeOldV1Files must match what was used to encode old files, cannot change
	
	UE_CLOG(Length < 0, LogSerialization, Fatal, TEXT(" Archive SerializedCompressed Length (%lld) < 0"), Length);

	if( IsLoading() )
	{
		// Serialize package file tag used to determine endianess.
		FCompressedChunkInfo PackageFileTag;
		PackageFileTag.CompressedSize	= 0;
		PackageFileTag.UncompressedSize	= 0;
		*this << PackageFileTag;

		// v1 header did not store CompressionFormatToDecode
		//	assume it was CompressionFormatToDecodeOldV1Files (usually Zlib)
		FName CompressionFormatToDecode = CompressionFormatToDecodeOldV1Files;

		bool bWasByteSwapped=false;
		bool bReadCompressionFormat=false;
		
		// FPackageFileSummary has int32 Tag == PACKAGE_FILE_TAG
		// this header does not otherwise match FPackageFileSummary in any way

		// low 32 bits of ARCHIVE_V2_HEADER_TAG are == PACKAGE_FILE_TAG
		#define ARCHIVE_V2_HEADER_TAG	(PACKAGE_FILE_TAG | ((uint64)0x22222222<<32) )

		if ( PackageFileTag.CompressedSize == PACKAGE_FILE_TAG )
		{
			// v1 header, not swapped
		}
		else if ( PackageFileTag.CompressedSize == PACKAGE_FILE_TAG_SWAPPED ||
			PackageFileTag.CompressedSize == BYTESWAP_ORDER64((uint64)PACKAGE_FILE_TAG) )
		{
			// v1 header, swapped
			bWasByteSwapped = true;
		}
		else if ( PackageFileTag.CompressedSize == ARCHIVE_V2_HEADER_TAG ||
			PackageFileTag.CompressedSize == BYTESWAP_ORDER64((uint64)ARCHIVE_V2_HEADER_TAG) )
		{
			// v2 header
			bWasByteSwapped = ( PackageFileTag.CompressedSize != ARCHIVE_V2_HEADER_TAG );
			bReadCompressionFormat = true;

			// read CompressionFormatToDecode
			FCompressionUtil::SerializeCompressorName(*this,CompressionFormatToDecode);
		}
		else
		{
			UE_LOGF(LogSerialization, Log, "ArchiveName: %ls", *GetArchiveName());
			UE_LOGF(LogSerialization, Log, "Archive UE Version: %d", UEVer().ToValue());
			UE_LOGF(LogSerialization, Log, "Archive Licensee Version: %d", LicenseeUEVer());
			UE_LOGF(LogSerialization, Log, "Position: %lld", Tell());
			UE_LOGF(LogSerialization, Log, "Read Size: %lld", Length);
			UE_LOGF(LogSerialization, Fatal, "BulkData compressed header read error. This package may be corrupt!");
		}
		
		if ( ! bReadCompressionFormat )
		{
			// upgrade old flag method
			if ((Flags & COMPRESS_DeprecatedFormatFlagsMask) != 0)
			{
				UE_LOGF(LogSerialization, Warning, "Old style compression flags are being used with FAsyncCompressionChunk, please update any code using this!");
				PRAGMA_DISABLE_DEPRECATION_WARNINGS
				CompressionFormatToDecode = FCompression::GetCompressionFormatFromDeprecatedFlags(Flags);
				PRAGMA_ENABLE_DEPRECATION_WARNINGS
			}

			if (CompressionFormatToDecode == NAME_Zlib && FPlatformProperties::GetZlibReplacementFormat() != nullptr)
			{
				// use this platform's replacement format in case it's not zlib
				CompressionFormatToDecode = FPlatformProperties::GetZlibReplacementFormat();
			}
		}
		else
		{		
			// shouldn't need to do this step for v2 headers; zlib should have already been changed by the encoder
			//	need to verify that's working right for xb1
			if (CompressionFormatToDecode == NAME_Zlib && FPlatformProperties::GetZlibReplacementFormat() != nullptr)
			{
				const char * ZlibReplacement = FPlatformProperties::GetZlibReplacementFormat();
				
				// go ahead and do it but warn :
				CompressionFormatToDecode = FName(ZlibReplacement);

				UE_LOGF(LogSerialization, Warning, "Archive v2 header with ZLib not ZlibReplacement: %ls", *CompressionFormatToDecode.ToString());
			}
		}

		// CompressionFormatToDecode came from disk, need to validate it :
		if ( ! FCompression::IsFormatValid(CompressionFormatToDecode) )
		{
			UE_LOGF(LogSerialization, Log, "ArchiveName: %ls", *GetArchiveName());
			UE_LOGF(LogSerialization, Log, "Archive UE Version: %d", UEVer().ToValue());
			UE_LOGF(LogSerialization, Log, "Archive Licensee Version: %d", LicenseeUEVer());
			UE_LOGF(LogSerialization, Log, "Position: %lld", Tell());
			UE_LOGF(LogSerialization, Log, "Read Size: %lld", Length);
			UE_LOGF(LogSerialization, Log, "CompressionFormatToDecode not found : %ls", *CompressionFormatToDecode.ToString());
			UE_LOGF(LogSerialization, Fatal, "BulkData compressed header read error. This package may be corrupt!");
		}

		// Read in base summary, contains total sizes :
		FCompressedChunkInfo Summary;
		*this << Summary;

		if (bWasByteSwapped)
		{
			Summary.CompressedSize = BYTESWAP_ORDER64(Summary.CompressedSize);
			Summary.UncompressedSize = BYTESWAP_ORDER64(Summary.UncompressedSize);
			PackageFileTag.UncompressedSize = BYTESWAP_ORDER64(PackageFileTag.UncompressedSize);
		}
		
		UE_CLOGF(Summary.CompressedSize < 0 || Summary.CompressedSize > (INT64_MAX/2) , LogSerialization, Fatal, " Archive SerializedCompressed CompressedSize (%lld) invalid", (int64)Summary.CompressedSize);
		UE_CLOGF(Summary.UncompressedSize < 0 || Summary.UncompressedSize > (INT64_MAX/2) , LogSerialization, Fatal, " Archive SerializedCompressed UncompressedSize (%lld) invalid", (int64)Summary.UncompressedSize);

		// Handle change in compression chunk size in backward compatible way.
		int64 LoadingCompressionChunkSize = PackageFileTag.UncompressedSize;
		if (LoadingCompressionChunkSize == PACKAGE_FILE_TAG)
		{
			LoadingCompressionChunkSize = LOADING_COMPRESSION_CHUNK_SIZE;
		}

		UE_CLOG(LoadingCompressionChunkSize <= 0, LogSerialization, Fatal, TEXT(" Archive SerializedCompressed LoadingCompressionChunkSize (%lld) <= 0"), LoadingCompressionChunkSize);
		UE_CLOGF(LoadingCompressionChunkSize >= INT32_MAX, LogSerialization, Fatal, " Archive SerializedCompressed LoadingCompressionChunkSize (%lld) >= INT32_MAX", LoadingCompressionChunkSize);

		// check Summary.UncompressedSize vs [V,Length] passed in
		if ( OutPartialReadLength == nullptr )
		{
			// UncompressedSize must == Length
			UE_CLOGF( Summary.UncompressedSize != Length, LogSerialization, Fatal, " Archive SerializedCompressed UncompressedSize (%lld) != Length (%lld)", (int64)Summary.UncompressedSize, (int64) Length );
		}
		else
		{
			// UncompressedSize must be <= Length and >= 0
			UE_CLOG( Summary.UncompressedSize > Length || Summary.UncompressedSize < 0, LogSerialization, Fatal, TEXT(" Archive SerializedCompressed UncompressedSize (%lld) > Length (%lld) or < 0"), (int64)Summary.UncompressedSize, (int64) Length );
			*OutPartialReadLength = Summary.UncompressedSize;
		}

		// Figure out how many chunks there are going to be based on uncompressed size and compression chunk size.
		//  divide and round up, safe without overflow due to previous range checks :
		int64	TotalChunkCount	= FMath::DivideAndRoundUp( Summary.UncompressedSize , LoadingCompressionChunkSize );

		// Allocate compression chunk infos and serialize them, keeping track of max size of compression chunks used.
		TArray64<FCompressedChunkInfo> CompressionChunks;
		CompressionChunks.SetNum(TotalChunkCount);
		int64 MaxCompressedSize	= 0;
		int64 TotalChunkCompressedSize = 0;
		int64 TotalChunkUncompressedSize = 0;
		for( int64 ChunkIndex=0; ChunkIndex<TotalChunkCount; ChunkIndex++ )
		{
			*this << CompressionChunks[ChunkIndex];
			if (bWasByteSwapped)
			{
				CompressionChunks[ChunkIndex].CompressedSize	= BYTESWAP_ORDER64( CompressionChunks[ChunkIndex].CompressedSize );
				CompressionChunks[ChunkIndex].UncompressedSize	= BYTESWAP_ORDER64( CompressionChunks[ChunkIndex].UncompressedSize );
			}
			
			UE_CLOG( CompressionChunks[ChunkIndex].CompressedSize < 0 || CompressionChunks[ChunkIndex].UncompressedSize < 0, 
				LogSerialization, Fatal, TEXT(" Archive SerializedCompressed CompressionChunks[ChunkIndex].CompressedSize (%lld) < 0 || CompressionChunks[ChunkIndex].UncompressedSize (%lld) < 0"), CompressionChunks[ChunkIndex].CompressedSize, CompressionChunks[ChunkIndex].UncompressedSize );

			MaxCompressedSize = FMath::Max( CompressionChunks[ChunkIndex].CompressedSize, MaxCompressedSize );

			TotalChunkCompressedSize += CompressionChunks[ChunkIndex].CompressedSize;
			TotalChunkUncompressedSize += CompressionChunks[ChunkIndex].UncompressedSize;
		}
		
		// verify the CompressionChunks[] sizes we read add up to the total we read
		UE_CLOGF( TotalChunkCompressedSize != Summary.CompressedSize, LogSerialization, Fatal, " Archive SerializedCompressed TotalChunkCompressedSize (%lld) != Summary.CompressedSize (%lld)", (int64)TotalChunkCompressedSize, (int64) Summary.CompressedSize );
		UE_CLOGF( TotalChunkUncompressedSize != Summary.UncompressedSize, LogSerialization, Fatal, " Archive SerializedCompressed TotalChunkUncompressedSize (%lld) != Summary.UnompressedSize (%lld)", (int64)TotalChunkUncompressedSize, (int64) Summary.UncompressedSize );

		// Set up destination pointer and allocate memory for compressed chunk[s] (one at a time).
		check( ! bTreatBufferAsFileReader );
		uint8*	Dest				= (uint8*) V;
		void*	CompressedBuffer	= FMemory::Malloc( MaxCompressedSize );

		// Iterate over all chunks, serialize them into memory and decompress them directly into the destination pointer
		for( int64 ChunkIndex=0; ChunkIndex<TotalChunkCount; ChunkIndex++ )
		{
			const FCompressedChunkInfo& Chunk = CompressionChunks[ChunkIndex];
			// Read compressed data.
			Serialize( CompressedBuffer, Chunk.CompressedSize );

			// check Serialize error before trying to decode
			if ( IsError() )
			{
				UE_LOGF(LogSerialization,Error, "Failed to serialize compress chunk in %ls, Chunk.CompressedSize=%d", *GetArchiveName(), (int)Chunk.CompressedSize);
				break;
			}

			// Decompress into dest pointer directly.
			bool bUncompressMemorySucceeded = FCompression::UncompressMemory( CompressionFormatToDecode, Dest, Chunk.UncompressedSize, CompressedBuffer, Chunk.CompressedSize, COMPRESS_NoFlags);

			if ( ! bUncompressMemorySucceeded )
			{
				UE_LOGF(LogSerialization,Error, "Failed to uncompress data in %ls, CompressionFormatToDecode=%ls", *GetArchiveName(), *CompressionFormatToDecode.ToString());
				SetError();
				break;
			}

			// And advance it by read amount.
			Dest += Chunk.UncompressedSize;
		}

		// Free up allocated memory.
		FMemory::Free( CompressedBuffer );
	}
	else if( IsSaving() )
	{
		SCOPE_SECONDS_COUNTER(GArchiveSerializedCompressedSavingTime);
		check( Length > 0 );

		// upgrade old flag method
		if ((Flags & COMPRESS_DeprecatedFormatFlagsMask) != 0)
		{
			check( CompressionFormatToEncode == NAME_Zlib );
			UE_LOGF(LogSerialization, Warning, "Old style compression flags are being used with FAsyncCompressionChunk, please update any code using this!");
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			CompressionFormatToEncode = FCompression::GetCompressionFormatFromDeprecatedFlags(Flags);
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}

		// if there's a cooking target, and it wants to replace Zlib compression with another format, use it. When loading, 
		// the platform will replace Zlib with that format above
		if (CompressionFormatToEncode == NAME_Zlib && IsCooking())
		{
			// use the replacement format
			CompressionFormatToEncode = CookingTarget()->GetZlibReplacementFormat();

			// with v2 headers, the modified CompressionFormatToEncode will be written in the archive
		}

		// compression chunk sizes must fit in int32 for old FCompression API
		//	(GSavingCompressionChunkSize is an int32 so this is automatic)
		check( GSavingCompressionChunkSize > 0 );
		check( GSavingCompressionChunkSize < INT32_MAX );
		// limit on maximum length we can serialize :
		check( Length <= (INT64_MAX/2) );

		// Serialize package file tag used to determine endianness in LoadCompressedData.
		FCompressedChunkInfo PackageFileTag;
		//PackageFileTag.CompressedSize	= PACKAGE_FILE_TAG;
		PackageFileTag.CompressedSize	= ARCHIVE_V2_HEADER_TAG;
		PackageFileTag.UncompressedSize	= GSavingCompressionChunkSize;
		*this << PackageFileTag;

		// v2 header writes compressor used :
		FCompressionUtil::SerializeCompressorName(*this,CompressionFormatToEncode);

		// Figure out how many chunks there are going to be based on uncompressed size and compression chunk size.
		//  divide and round up, overflow safe due to previous range checks
		int64	TotalChunkCount	= FMath::DivideAndRoundUp( Length, (int64)GSavingCompressionChunkSize );

		//  +1 for Summary chunk	
		TotalChunkCount += 1;

		// Keep track of current position so we can later seek back and overwrite stub compression chunk infos.
		int64 StartPosition = Tell();

		// Allocate compression chunk infos and serialize them with default fields so we can later overwrite the data.
		TArray64<FCompressedChunkInfo> CompressionChunks;
		CompressionChunks.SetNum(TotalChunkCount);
		for( int64 ChunkIndex=0; ChunkIndex<TotalChunkCount; ChunkIndex++ )
		{
			*this << CompressionChunks[ChunkIndex];
		}

		// The uncompressd size is equal to the passed in length.
		CompressionChunks[0].UncompressedSize	= Length;
		// Zero initialize compressed size so we can update it during chunk compression.
		CompressionChunks[0].CompressedSize		= 0;

#if WITH_MULTI_THREADED_COMPRESSION

#define MAX_COMPRESSION_JOBS (16)
		// Don't scale more than 16x to avoid going overboard wrt temporary memory.
		FAsyncTask<FAsyncCompressionChunk> AsyncChunks[MAX_COMPRESSION_JOBS];

		// used to keep track of which job is the next one we need to retire
		int64 AsyncChunkIndex[MAX_COMPRESSION_JOBS]={0};

		static uint32 GNumUnusedThreads_SerializeCompressed = -1;
		if (GNumUnusedThreads_SerializeCompressed == (uint32)-1)
		{
			// one-time initialization
			GNumUnusedThreads_SerializeCompressed = 1;
			// if we should use all available cores then we want to compress with all
			if( FParse::Param(FCommandLine::Get(), TEXT("USEALLAVAILABLECORES")) == true )
			{
				GNumUnusedThreads_SerializeCompressed = 0;
			}
		}

		// Maximum number of concurrent async tasks we're going to kick off. This is based on the number of processors
		// available in the system.
		int32 MaxConcurrentAsyncChunks = FMath::Clamp<int32>( FPlatformMisc::NumberOfCores() - GNumUnusedThreads_SerializeCompressed, 1, MAX_COMPRESSION_JOBS );
		if (FParse::Param(FCommandLine::Get(), TEXT("MTCHILD")))
		{
			// throttle this back when doing MT cooks
			MaxConcurrentAsyncChunks = FMath::Min<int32>( MaxConcurrentAsyncChunks,4 );
		}

		// Number of chunks left to finalize.
		int64 NumChunksLeftToFinalize	= TotalChunkCount -1; // -1 for summary chunk
		// Number of chunks left to kick off
		int64 NumChunksLeftToKickOff	= NumChunksLeftToFinalize;
		// Start at index 1 as first chunk info is summary.
		int64	CurrentChunkIndex		= 1;
		// Start at index 1 as first chunk info is summary.
		int64	RetireChunkIndex		= 1;
	
		// Number of bytes remaining to kick off compression for.
		int64 BytesRemainingToKickOff	= Length;
		// Pointer to src data if buffer is memory pointer, NULL if it's a FArchive.
		uint8* SrcBuffer = bTreatBufferAsFileReader ? NULL : (uint8*)V;

		check(!bTreatBufferAsFileReader || ((FArchive*)V)->IsLoading());
		check(NumChunksLeftToFinalize);

		// Loop while there is work left to do based on whether we have finalized all chunks yet.
		while( NumChunksLeftToFinalize )
		{
			// If true we are waiting for async tasks to complete and should wait to complete some
			// if there are no async tasks finishing this iteration.
			bool bNeedToWaitForAsyncTask = false;

			// Try to kick off async tasks if there are chunks left to kick off.
			if( NumChunksLeftToKickOff )
			{
				// Find free index based on looking at uncompressed size. We can't use the thread counter
				// for this as that might be a chunk ready for finalization.
				int32 FreeIndex = INDEX_NONE;
				for( int32 i=0; i<MaxConcurrentAsyncChunks; i++ )
				{
					if( !AsyncChunkIndex[i] )
					{
						FreeIndex = i;
						check(AsyncChunks[FreeIndex].IsIdle()); // this is not supposed to be in use
						break;
					}
				}

				// Kick off async compression task if we found a chunk for it.
				if( FreeIndex != INDEX_NONE )
				{
					FAsyncCompressionChunk& NewChunk = AsyncChunks[FreeIndex].GetTask();

					NewChunk.CompressedSize	= FCompression::CompressMemoryBound(CompressionFormatToEncode, GSavingCompressionChunkSize);
					// Allocate compressed buffer placeholder on first use.
					if( NewChunk.CompressedBuffer == NULL )
					{
						NewChunk.CompressedBuffer = FMemory::Malloc( NewChunk.CompressedSize	);
					}

					// By default everything is chunked up into GSavingCompressionChunkSize chunks.
					NewChunk.UncompressedSize	= FMath::Min( BytesRemainingToKickOff, (int64)GSavingCompressionChunkSize );
					check(NewChunk.UncompressedSize>0);

					// Need to serialize source data if passed in pointer is an FArchive.
					if( bTreatBufferAsFileReader )
					{
						// Allocate memory on first use. We allocate the maximum amount to allow reuse.
						if( !NewChunk.UncompressedBuffer )
						{
							NewChunk.UncompressedBuffer = FMemory::Malloc(GSavingCompressionChunkSize);
						}
						((FArchive*)V)->Serialize(NewChunk.UncompressedBuffer, NewChunk.UncompressedSize);
					}
					else
					{
						// Advance src pointer by amount to be compressed.
						NewChunk.UncompressedBuffer = SrcBuffer;
						SrcBuffer += NewChunk.UncompressedSize;
					}

					// Update status variables for tracking how much work is left, what to do next.
					BytesRemainingToKickOff -= NewChunk.UncompressedSize;
					AsyncChunkIndex[FreeIndex] = CurrentChunkIndex++;
					NewChunk.Flags = Flags;
					NewChunk.CompressionFormat = CompressionFormatToEncode;
					NumChunksLeftToKickOff--;

					AsyncChunks[FreeIndex].StartBackgroundTask();
				}
				else
				{
					// No chunks were available to use, complete some
					bNeedToWaitForAsyncTask = true;
				}
			}

			// Wait for the oldest task to finish instead of spinning
			if (NumChunksLeftToKickOff == 0)
			{
				bNeedToWaitForAsyncTask = true;
			}

			// Index of oldest chunk, needed as we need to serialize in order.
			int32 OldestAsyncChunkIndex = INDEX_NONE;
			for( int32 i=0; i<MaxConcurrentAsyncChunks; i++ )
			{
				check(AsyncChunkIndex[i] == 0 || AsyncChunkIndex[i] >= RetireChunkIndex);
				check(AsyncChunkIndex[i] < RetireChunkIndex + MaxConcurrentAsyncChunks);
				if (AsyncChunkIndex[i] == RetireChunkIndex)
				{
					OldestAsyncChunkIndex = i;
				}
			}
			check(OldestAsyncChunkIndex != INDEX_NONE);  // the retire chunk better be outstanding


			bool ChunkReady;
			if (bNeedToWaitForAsyncTask)
			{
				// This guarantees that the async work has finished, doing it on this thread if it hasn't been started
				AsyncChunks[OldestAsyncChunkIndex].EnsureCompletion();
				ChunkReady = true;
			}
			else
			{
				ChunkReady = AsyncChunks[OldestAsyncChunkIndex].IsDone();
			}
			if (ChunkReady)
			{
				FAsyncCompressionChunk& DoneChunk = AsyncChunks[OldestAsyncChunkIndex].GetTask();
				// Serialize the data via archive.
				Serialize( DoneChunk.CompressedBuffer, DoneChunk.CompressedSize );

				// Update associated chunk.
				int64 CompressionChunkIndex = RetireChunkIndex++;
				check(CompressionChunkIndex<TotalChunkCount);
				CompressionChunks[CompressionChunkIndex].CompressedSize		= DoneChunk.CompressedSize;
				CompressionChunks[CompressionChunkIndex].UncompressedSize	= DoneChunk.UncompressedSize;

				// Keep track of total compressed size, stored in first chunk.
				CompressionChunks[0].CompressedSize	+= DoneChunk.CompressedSize;

				// Clean up chunk. Src and dst buffer are not touched as the contain allocations we keep till the end.
				AsyncChunkIndex[OldestAsyncChunkIndex] = 0;
				DoneChunk.CompressedSize	= 0;
				DoneChunk.UncompressedSize = 0;

				// Finalized one :)
				NumChunksLeftToFinalize--;
				bNeedToWaitForAsyncTask = false;
			}
		}

		// Free intermediate buffer storage.
		for( int32 i=0; i<MaxConcurrentAsyncChunks; i++ )
		{
			// Free temporary compressed buffer storage.
			FMemory::Free( AsyncChunks[i].GetTask().CompressedBuffer );
			AsyncChunks[i].GetTask().CompressedBuffer = NULL;
			// Free temporary uncompressed buffer storage if data was serialized in.
			if( bTreatBufferAsFileReader )
			{
				FMemory::Free( AsyncChunks[i].GetTask().UncompressedBuffer );
				AsyncChunks[i].GetTask().UncompressedBuffer = NULL;
			}
		}

#else
		// Set up source pointer amount of data to copy (in bytes)
		uint8*	Src;
		// allocate memory to read into
		if (bTreatBufferAsFileReader)
		{
			Src = (uint8*)FMemory::Malloc(GSavingCompressionChunkSize);
			check(((FArchive*)V)->IsLoading());
		}
		else
		{
			Src = (uint8*) V;
		}
		int64		BytesRemaining			= Length;
		// Start at index 1 as first chunk info is summary.
		int64		CurrentChunkIndex		= 1;

		int64		CompressedBufferSize	= FCompression::CompressMemoryBound(CompressionFormatToEncode, GSavingCompressionChunkSize);
		void*	CompressedBuffer		= FMemory::Malloc( CompressedBufferSize );

		while( BytesRemaining > 0 )
		{
			int64 BytesToCompress = FMath::Min( BytesRemaining, (int64)GSavingCompressionChunkSize );
			int64 CompressedSize	= CompressedBufferSize;

			// read in the next chunk from the reader
			if (bTreatBufferAsFileReader)
			{
				((FArchive*)V)->Serialize(Src, BytesToCompress);
			}

			check(CompressedSize < INT_MAX);
			int32 CompressedSizeInt = (int32)CompressedSize;
						
			verify( FCompression::CompressMemory( CompressionFormatToEncode, CompressedBuffer, CompressedSizeInt, Src, BytesToCompress, Flags ) );
			CompressedSize = CompressedSizeInt;
			// move to next chunk if not reading from file
			if (!bTreatBufferAsFileReader)
			{
				Src += BytesToCompress;
			}
			Serialize( CompressedBuffer, CompressedSize );
			// Keep track of total compressed size, stored in first chunk.
			CompressionChunks[0].CompressedSize	+= CompressedSize;

			// Update current chunk.
			check(CurrentChunkIndex<TotalChunkCount);
			CompressionChunks[CurrentChunkIndex].CompressedSize		= CompressedSize;
			CompressionChunks[CurrentChunkIndex].UncompressedSize	= BytesToCompress;
			CurrentChunkIndex++;
			
			BytesRemaining -= GSavingCompressionChunkSize;
		}

		// free the buffer we read into
		if (bTreatBufferAsFileReader)
		{
			FMemory::Free(Src);
		}

		// Free allocated memory.
		FMemory::Free( CompressedBuffer );
#endif

		// Overrwrite chunk infos by seeking to the beginning, serializing the data and then
		// seeking back to the end.
		auto EndPosition = Tell();
		// Seek to the beginning.
		Seek( StartPosition );
		// Serialize chunk infos.
		for( int64 ChunkIndex=0; ChunkIndex<TotalChunkCount; ChunkIndex++ )
		{
			*this << CompressionChunks[ChunkIndex];
		}
		// Seek back to end.
		Seek( EndPosition );
	}
}

FArchive::~FArchive() = default;

void FArchive::ByteSwap(void* V, int32 Length)
{
	uint8* Ptr = (uint8*)V;
	int32 Top = Length - 1;
	int32 Bottom = 0;
	while (Bottom < Top)
	{
		Swap(Ptr[Top--], Ptr[Bottom++]);
	}
}

FArchive& FArchive::SerializeByteOrderSwapped(void* V, int32 Length)
{
	if (IsLoading())
	{
		Serialize(V, Length); // Read.
		ByteSwap(V, Length); // Swap.
	}
	else // Writing
	{
		ByteSwap(V, Length); // Swap V.
		Serialize(V, Length); // Write V.
		ByteSwap(V, Length); // Swap V back to its original byte order to prevent caller from observing V swapped.
	}

	return *this;
}

FArchive& FArchive::SerializeByteOrderSwapped(uint16& Value)
{
	return ArchiveUtil::SerializeByteOrderSwapped(*this, Value);
}

FArchive& FArchive::SerializeByteOrderSwapped(uint32& Value)
{
	return ArchiveUtil::SerializeByteOrderSwapped(*this, Value);
}

FArchive& FArchive::SerializeByteOrderSwapped(uint64& Value)
{
	return ArchiveUtil::SerializeByteOrderSwapped(*this, Value);
}

void FArchive::SerializeIntPacked(uint32& Value)
{
	if (IsLoading())
	{
		// Set the byte cap to 10 in order to handle serialized 64-bit integers whose type changed to 32-bit.
		constexpr size_t MaxBytes = 10;

		uint32 TempValue = 0;
		uint8 cnt = 0;
		uint8 more = 1;
		while(more)
		{
			if (!ensureMsgf(cnt < MaxBytes, TEXT("SerializeIntPacked tried to load more than %zu packed bytes for a 32-bit value"), MaxBytes))
			{
				SetError();
				return;
			}

			uint8 NextByte;
			Serialize(&NextByte, 1);			// Read next byte

			more = NextByte & 1;				// Check 1 bit to see if theres more after this
			NextByte = NextByte >> 1;			// Shift to get actual 7 bit value
			TempValue += (uint32)((uint64)NextByte << (7 * cnt++));	// Add to total value
		}

		Value = TempValue;
	}
	else
	{
		constexpr size_t MaxBytes = 5;

		uint8 PackedBytes[MaxBytes];
		int32 PackedByteCount = 0;
		uint32 Remaining = Value;
		while(true)
		{
			uint8 nextByte = Remaining & 0x7f;		// Get next 7 bits to write
			Remaining = Remaining >> 7;				// Update remaining
			nextByte = nextByte << 1;				// Make room for 'more' bit
			if( Remaining > 0)
			{
				nextByte |= 1;						// set more bit
				PackedBytes[PackedByteCount++] = nextByte;
			}
			else
			{
				PackedBytes[PackedByteCount++] = nextByte;
				break;
			}
		}
		Serialize(PackedBytes, PackedByteCount); // Actually serialize the bytes we made
	}
}

void FArchive::SerializeIntPacked64(uint64& Value)
{
	constexpr size_t MaxBytes = 10;

	if (IsLoading())
	{
		uint64 TempValue = 0;
		uint8 cnt = 0;
		uint8 more = 1;
		while (more)
		{
			if (!ensureMsgf(cnt < MaxBytes, TEXT("SerializeIntPacked64 tried to load more than %zu packed bytes for a 64-bit value"), MaxBytes))
			{
				SetError();
				return;
			}

			uint8 NextByte;
			Serialize(&NextByte, 1);			// Read next byte

			more = NextByte & 1;				// Check 1 bit to see if theres more after this
			NextByte = NextByte >> 1;			// Shift to get actual 7 bit value
			TempValue += (uint64)NextByte << (7 * cnt++);	// Add to total value
		}

		Value = TempValue;
	}
	else
	{
		uint8 PackedBytes[MaxBytes];
		int32 PackedByteCount = 0;
		uint64 Remaining = Value;
		while (true)
		{
			uint8 nextByte = Remaining & 0x7f;		// Get next 7 bits to write
			Remaining = Remaining >> 7;				// Update remaining
			nextByte = nextByte << 1;				// Make room for 'more' bit
			if (Remaining > 0)
			{
				nextByte |= 1;						// set more bit
				PackedBytes[PackedByteCount++] = nextByte;
			}
			else
			{
				PackedBytes[PackedByteCount++] = nextByte;
				break;
			}
		}
		Serialize(PackedBytes, PackedByteCount); // Actually serialize the bytes we made
	}
}

void FArchive::LogfImpl(const TCHAR* Fmt, ...)
{
	// We need to use malloc here directly as GMalloc might not be safe, e.g. if called from within GMalloc!
	int32		BufferSize	= 1024;
	TCHAR*	Buffer		= NULL;
	int32		Result		= -1;

	while(Result == -1)
	{
		FMemory::SystemFree(Buffer);
		Buffer = (TCHAR*) FMemory::SystemMalloc( BufferSize * sizeof(TCHAR) );
		GET_TYPED_VARARGS_RESULT( TCHAR, Buffer, BufferSize, BufferSize-1, Fmt, Fmt, Result );
		BufferSize *= 2;
	};
	Buffer[Result] = TEXT('\0');

	// Convert to ANSI and serialize as ANSI char.
	for( int32 i=0; i<Result; i++ )
	{
		ANSICHAR Char = CharCast<ANSICHAR>( Buffer[i] );
		Serialize( &Char, 1 );
	}

	// Write out line terminator.
	for( int32 i=0; LINE_TERMINATOR[i]; i++ )
	{
		ANSICHAR Char = LINE_TERMINATOR[i];
		Serialize( &Char, 1 );
	}

	// Free temporary buffers.
	FMemory::SystemFree( Buffer );
}

void FArchiveState::ThisRequiresLocalizationGather()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	ForEachState([](FArchiveState& State) { State.ArRequiresLocalizationGather = true; });
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void FArchiveState::SetUEVer(FPackageFileVersion InVer)
{
	ArUEVer = InVer;
}

void FArchiveState::SetLicenseeUEVer(int32 InVer)
{
	ArLicenseeUEVer = InVer;
}

void FArchiveState::SetEngineVer(const FEngineVersionBase& InVer)
{
	ArEngineVer = InVer;
}

void FArchiveState::SetEngineNetVer(const uint32 InEngineNetVer)
{
	static const FName EngineNetworkVersionName("EngineNetworkVersion");
	SetCustomVersion(FEngineNetworkCustomVersion::Guid, InEngineNetVer, EngineNetworkVersionName);
}

uint32 FArchiveState::EngineNetVer() const
{
	return CustomVer(FEngineNetworkCustomVersion::Guid);
}

void FArchiveState::SetGameNetVer(const uint32 InGameNetVer)
{
	static const FName GameNetworkVersionName("GameNetworkVersion");
	SetCustomVersion(FGameNetworkCustomVersion::Guid, InGameNetVer, GameNetworkVersionName);
}

uint32 FArchiveState::GameNetVer() const
{
	return CustomVer(FGameNetworkCustomVersion::Guid);
}

void FArchiveState::SetIsLoading(bool bInIsLoading)
{
	ArIsLoading = bInIsLoading;
}

void FArchiveState::SetIsLoadingFromCookedPackage(bool bInIsLoadingFromCookedPackage)
{
	ArIsLoadingFromCookedPackage = bInIsLoadingFromCookedPackage;
}

void FArchiveState::SetIsSaving(bool bInIsSaving)
{
	ArIsSaving = bInIsSaving;
}

void FArchiveState::SetIsTransacting(bool bInIsTransacting)
{
	ArIsTransacting = bInIsTransacting;
}

void FArchiveState::SetIsTextFormat(bool bInIsTextFormat)
{
	ArIsTextFormat = bInIsTextFormat;
}

void FArchiveState::SetWantBinaryPropertySerialization(bool bInWantBinaryPropertySerialization)
{
	ArWantBinaryPropertySerialization = bInWantBinaryPropertySerialization;
}

void FArchiveState::SetUseUnversionedPropertySerialization(bool bInUseUnversioned)
{
	ArUseUnversionedPropertySerialization = bInUseUnversioned;
}

void FArchiveState::SetForceUnicode(bool bInForceUnicode)
{
	ArForceUnicode = bInForceUnicode;
}

void FArchiveState::SetIsPersistent(bool bInIsPersistent)
{
	ArIsPersistent = bInIsPersistent;
}

static_assert(sizeof(FArchive) == sizeof(FArchiveState), "New FArchive members should be added to FArchiveState instead");

PRAGMA_RESTORE_UNSAFE_TYPECAST_WARNINGS
