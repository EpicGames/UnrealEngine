// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/StructuredArchive.h"
#include "Internationalization/Text.h"

#if WITH_TEXT_ARCHIVE_SUPPORT

struct FArchiveFromStructuredArchiveImpl::FImpl
{
	explicit FImpl(FStructuredArchive::FSlot Slot)
		: RootSlot(Slot)
	{
	}

	TOptional<FStructuredArchive::FRecord> Root;

	static constexpr int32 MaxBufferSize = 128;

	bool bPendingSerialize = false;
	bool bWasOpened        = false;

	TArray<uint8> Buffer;
	int32 Pos = 0;

	TArray<FName> Names;
	TMap<FName, int32> NameToIndex;

	TArray<FString> ObjectNames;
	TArray<UObject*> Objects;
	TBitArray<> ObjectsValid;
	TMap<UObject*, int32> ObjectToIndex;

	FStructuredArchive::FSlot RootSlot;
};

FArchiveFromStructuredArchiveImpl::FArchiveFromStructuredArchiveImpl(FStructuredArchive::FSlot Slot)
	: FArchiveProxy(Slot.GetUnderlyingArchive())
	, Pimpl(Slot)
{
	// For some reason, the FArchive copy constructor will copy all the trivial members of the source archive, but then specifically set ArIsFilterEditorOnly to false, with a comment saying
	// they don't know why it's doing this... make sure we inherit this flag here!
	ArIsFilterEditorOnly = InnerArchive.ArIsFilterEditorOnly;
	SetIsTextFormat(false);
}

FArchiveFromStructuredArchiveImpl::~FArchiveFromStructuredArchiveImpl()
{
	checkf(!Pimpl->bPendingSerialize, TEXT("Archive adapters must be closed before destruction"));
}

void FArchiveFromStructuredArchiveImpl::Flush()
{
	Commit();
	FArchive::Flush();
}

bool FArchiveFromStructuredArchiveImpl::Close()
{
	Commit();
	return FArchive::Close();
}

int64 FArchiveFromStructuredArchiveImpl::Tell()
{
	if (InnerArchive.IsTextFormat())
	{
		return Pimpl->Pos;
	}
	else
	{
		return InnerArchive.Tell();
	}
}

int64 FArchiveFromStructuredArchiveImpl::TotalSize()
{
	checkf(false, TEXT("FArchiveFromStructuredArchive does not support TotalSize()"));
	return FArchive::TotalSize();
}

void FArchiveFromStructuredArchiveImpl::Seek(int64 InPos)
{
	if (InnerArchive.IsTextFormat())
	{
		check(Pimpl->Pos >= 0 && Pimpl->Pos <= Pimpl->Buffer.Num());
		Pimpl->Pos = (int32)InPos;
	}
	else
	{
		InnerArchive.Seek(InPos);
	}
}

bool FArchiveFromStructuredArchiveImpl::AtEnd()
{
	if (InnerArchive.IsTextFormat())
	{
		return Pimpl->Pos == Pimpl->Buffer.Num();
	}
	else
	{
		return InnerArchive.AtEnd();
	}
}

FArchive& FArchiveFromStructuredArchiveImpl::operator<<(class FName& Value)
{
	OpenArchive();

	if (InnerArchive.IsTextFormat())
	{
		if (IsLoading())
		{
			int32 NameIdx = 0;
			Serialize(&NameIdx, sizeof(NameIdx));
			Value = Pimpl->Names[NameIdx];
		}
		else
		{
			int32* NameIdxPtr = Pimpl->NameToIndex.Find(Value);
			if (NameIdxPtr == nullptr)
			{
				NameIdxPtr = &Pimpl->NameToIndex.Add(Value);
				*NameIdxPtr = Pimpl->Names.Add(Value);
			}
			Serialize(NameIdxPtr, sizeof(*NameIdxPtr));
		}
	}
	else
	{
		InnerArchive << Value;
	}
	return *this;
}

FArchive& FArchiveFromStructuredArchiveImpl::operator<<(class UObject*& Value)
{
	OpenArchive();

	if(InnerArchive.IsTextFormat())
	{
		if (IsLoading())
		{
			int32 ObjectIdx = 0;
			Serialize(&ObjectIdx, sizeof(ObjectIdx));

			// If this object has already been accessed, return the cached value
			if (Pimpl->ObjectsValid[ObjectIdx])
			{
				Value = Pimpl->Objects[ObjectIdx];
			}
			else
			{
				FStructuredArchive::FStream Stream = Pimpl->Root->EnterStream(TEXT("Objects"));

				// Skip earlier elements
				FString Str;
				for (int32 Index = 0; Index < ObjectIdx; ++Index)
				{
					Stream.EnterElement() << Str;
				}

				Stream.EnterElement() << Value;
				Pimpl->Objects[ObjectIdx] = Value;
				Pimpl->ObjectsValid[ObjectIdx] = true;
			}
		}
		else
		{
			int32* ObjectIdxPtr = Pimpl->ObjectToIndex.Find(Value);
			if (ObjectIdxPtr == nullptr)
			{
				ObjectIdxPtr = &Pimpl->ObjectToIndex.Add(Value);
				*ObjectIdxPtr = Pimpl->Objects.Add(Value);
			}
			Serialize(ObjectIdxPtr, sizeof(*ObjectIdxPtr));
		}
	}
	else
	{
		InnerArchive << Value;
	}
	return *this;
}

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
FArchive& FArchiveFromStructuredArchiveImpl::operator<<(Verse::VCell*& Value)
{
	OpenArchive();

	if (InnerArchive.IsTextFormat())
	{
		// VCell references are not generally supported in structured text archives, but
		// ignoring them while saving makes things easier for use cases like debug printing.
		check(!IsLoading());
	}
	else
	{
		InnerArchive << Value;
	}
	return *this;
}
#endif

FArchive& FArchiveFromStructuredArchiveImpl::operator<<(class FText& Value)
{
	OpenArchive();

	if (InnerArchive.IsTextFormat())
	{
		FText::SerializeText(*this, Value);
	}
	else
	{
		InnerArchive << Value;
	}
	return *this;
}

void FArchiveFromStructuredArchiveImpl::Serialize(void* V, int64 Length)
{
	OpenArchive();

	if (InnerArchive.IsTextFormat())
	{
		if (IsLoading())
		{
			if (Pimpl->Pos + Length > Pimpl->Buffer.Num())
			{
				checkf(false, TEXT("Attempt to read past end of archive"));
			}
			FMemory::Memcpy(V, Pimpl->Buffer.GetData() + Pimpl->Pos, Length);
			Pimpl->Pos += (int32)Length;
		}
		else
		{
			if (Pimpl->Pos + Length > Pimpl->Buffer.Num())
			{
				Pimpl->Buffer.AddUninitialized((int32)(Pimpl->Pos + Length - Pimpl->Buffer.Num()));
			}
			FMemory::Memcpy(Pimpl->Buffer.GetData() + Pimpl->Pos, V, Length);
			Pimpl->Pos += (int32)Length;
		}
	}
	else
	{
		InnerArchive.Serialize(V, Length);
	}
}

void FArchiveFromStructuredArchiveImpl::Commit()
{
	if (Pimpl->bWasOpened && InnerArchive.IsTextFormat())
	{
		Finalize(Pimpl->Root.GetValue());
	}
}

bool FArchiveFromStructuredArchiveImpl::Finalize(FStructuredArchive::FRecord Record)
{
	check(Pimpl->bWasOpened);
	bool bShouldSerialize = Pimpl->bPendingSerialize;
	Pimpl->bPendingSerialize = false;

	if (bShouldSerialize)
	{
		FStructuredArchive::FSlot DataSlot = Record.EnterField(TEXT("Data"));
		DataSlot.Serialize(Pimpl->Buffer);

		TOptional<FStructuredArchive::FSlot> ObjectsSlot = Record.TryEnterField(TEXT("Objects"), Pimpl->Objects.Num() > 0);
		if (ObjectsSlot.IsSet())
		{
			if (IsLoading())
			{
				// We don't want to load all the referenced objects here, as this causes all sorts of 
				// dependency issues. The legacy archive would load any referenced objects at the point
				// that their pointer was serialized by the owning export. For now, we just need to
				// know how many objects there are so we can pre-size our arrays
				// NOTE: The json formatter will push all the values in the array onto the value stack
				// when we enter the array here. We never read them, so I'm assuming they just sit there
				// until we destroy this archive wrapper. Perhaps we need something in the API here to just access
				// the size of the array but not preparing to access it's values?
				ObjectsSlot.GetValue() << Pimpl->ObjectNames;
				//int32 NumEntries = 0;
				//ObjectsSlot.GetValue().EnterArray(NumEntries);
				Pimpl->Objects.AddUninitialized(Pimpl->ObjectNames.Num());
				Pimpl->ObjectsValid.Init(false, Pimpl->ObjectNames.Num());
			}
			else
			{
				ObjectsSlot.GetValue() << Pimpl->Objects;
			}
		}

		TOptional<FStructuredArchive::FSlot> NamesSlot = Record.TryEnterField(TEXT("Names"), Pimpl->Names.Num() > 0);
		if (NamesSlot.IsSet())
		{
			NamesSlot.GetValue() << Pimpl->Names;
		}
	}

	return bShouldSerialize;
}

void FArchiveFromStructuredArchiveImpl::OpenArchive()
{
	if (!Pimpl->bWasOpened)
	{
		Pimpl->bWasOpened = true;

		if (InnerArchive.IsTextFormat())
		{
			Pimpl->bPendingSerialize = true;
			Pimpl->Root = Pimpl->RootSlot.EnterRecord();

			if (IsLoading())
			{
				Finalize(Pimpl->Root.GetValue());
			}
		}
		else
		{
			Pimpl->RootSlot.EnterStream();
		}
	}
}

FArchive* FArchiveFromStructuredArchiveImpl::GetCacheableArchive()
{
	if (IsTextFormat())
	{
		return nullptr;
	}
	else
	{
		return InnerArchive.GetCacheableArchive();
	}
}

bool FArchiveFromStructuredArchiveImpl::ContainsData() const
{
	return Pimpl->Buffer.Num() > 0;
}

#endif
