// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/ArchiveLoadCompressedProxy.h"

#include "Containers/Array.h"
#include "HAL/UnrealMemory.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Compression.h"

PRAGMA_DISABLE_UNSAFE_TYPECAST_WARNINGS

/*----------------------------------------------------------------------------
	FArchiveLoadCompressedProxy
----------------------------------------------------------------------------*/

FArchiveLoadCompressedProxy::FArchiveLoadCompressedProxy(const TArray<uint8>& InCompressedData, FName InCompressionFormat, ECompressionFlags InCompressionFlags)
	:	CompressedData(InCompressedData)
	,	CompressionFormat(InCompressionFormat)
	,	CompressionFlags(InCompressionFlags)
{
	this->SetIsLoading(true);
	this->SetIsPersistent(true);
	this->SetWantBinaryPropertySerialization(true);
	bShouldSerializeFromArray			= false;
	RawBytesSerialized					= 0;
	CurrentIndex						= 0;

	// Allocate temporary memory.
	TmpDataStart	= (uint8*) FMemory::Malloc(LOADING_COMPRESSION_CHUNK_SIZE);
	TmpDataEnd		= TmpDataStart + LOADING_COMPRESSION_CHUNK_SIZE;
	TmpData			= TmpDataEnd;
}

FArchiveLoadCompressedProxy::~FArchiveLoadCompressedProxy()
{
	// Free temporary memory allocated.
	FMemory::Free( TmpDataStart );
	TmpDataStart	= NULL;
	TmpDataEnd		= NULL;
	TmpData			= NULL;
}

/**
 * Flushes tmp data to array.
 */
void FArchiveLoadCompressedProxy::DecompressMoreData()
{
	if ( IsError() )
	{
		// don't try to decode more if we hit error
		TmpData = TmpDataStart;
		TmpDataEnd = TmpDataStart;
		return;
	}

	// This will call Serialize so we need to indicate that we want to serialize from array.
	bShouldSerializeFromArray = true;
	int64 DecompressedLength = 0;
	SerializeCompressedNew( TmpDataStart, LOADING_COMPRESSION_CHUNK_SIZE, CompressionFormat,CompressionFormat, CompressionFlags, false, &DecompressedLength);
	// last chunk will be partial :
	//	all chunks before last should have size == LOADING_COMPRESSION_CHUNK_SIZE
	check( DecompressedLength <= LOADING_COMPRESSION_CHUNK_SIZE );
	bShouldSerializeFromArray = false;
	// Buffer is filled again, reset.
	TmpData = TmpDataStart;
	TmpDataEnd = TmpDataStart + DecompressedLength;
}

/**
 * Serializes data from archive. This function is called recursively and determines where to serialize
 * from and how to do so based on internal state.
 *
 * @param	InData	Pointer to serialize to
 * @param	Count	Number of bytes to read
 */
void FArchiveLoadCompressedProxy::Serialize( void* InData, int64 Count )
{
	uint8* DstData = (uint8*) InData;

	if( bShouldSerializeFromArray )
	{
		// SerializedCompressed reads the compressed data from here
		check(CurrentIndex+Count<=CompressedData.Num());
		FMemory::Memcpy( DstData, &CompressedData[CurrentIndex], Count );
		CurrentIndex += Count;
	}
	// Regular call to serialize, read from temp buffer
	else
	{	
		while( Count )
		{
			int32 BytesToCopy = FMath::Min<int32>( Count, (int32)(TmpDataEnd - TmpData) );
			// Enough room in buffer to copy some data.
			if( BytesToCopy )
			{
				// We pass in a NULL pointer when forward seeking. In that case we don't want
				// to copy the data but only care about pointing to the proper spot.
				if( DstData )
				{
					FMemory::Memcpy( DstData, TmpData, BytesToCopy );
					DstData += BytesToCopy;
				}
				Count -= BytesToCopy;
				TmpData += BytesToCopy;
				RawBytesSerialized += BytesToCopy;
			}
			// Tmp buffer fully exhausted, decompress new one.
			else
			{
				// Decompress more data. This will call Serialize again so we need to handle recursion.
				DecompressMoreData();

				if ( TmpDataEnd == TmpData )
				{
					// wanted more but couldn't get any
					// avoid infinite loop
					SetError();
					return;
				}
			}
		}
	}
}

/**
 * Seeks to the passed in position in the stream. This archive only supports forward seeking
 * and implements it by serializing data till it reaches the position.
 */
void FArchiveLoadCompressedProxy::Seek( int64 InPos )
{
	int64 CurrentPos = Tell();
	int64 Difference = InPos - CurrentPos;
	// We only support forward seeking.
	check(Difference>=0);
	// Seek by serializing data, albeit with NULL destination so it's just decompressing data.
	Serialize( NULL, Difference );
}

/**
 * @return current position in uncompressed stream in bytes.
 */
int64 FArchiveLoadCompressedProxy::Tell()
{
	return RawBytesSerialized;
}

PRAGMA_RESTORE_UNSAFE_TYPECAST_WARNINGS
