// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/ArchiveProxy.h"

/*-----------------------------------------------------------------------------
	FArchiveProxy implementation.
-----------------------------------------------------------------------------*/

FArchiveProxy::FArchiveProxy(FArchive& InInnerArchive)
: FArchive    (InInnerArchive)
, InnerArchive(InInnerArchive)
{
	LinkProxy(InnerArchive.GetArchiveState(), GetArchiveState());
}

FArchiveProxy::~FArchiveProxy()
{
	UnlinkProxy(InnerArchive.GetArchiveState(), GetArchiveState());
}
