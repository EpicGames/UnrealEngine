// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/ArchiveSaveCompressedProxy.h"

#include "CoreGlobals.h"
#include "HAL/UnrealMemory.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/Compression.h"
#include "Trace/Detail/Channel.h"

PRAGMA_DISABLE_UNSAFE_TYPECAST_WARNINGS

/*----------------------------------------------------------------------------
	FArchiveSaveCompressedProxy
----------------------------------------------------------------------------*/
FArchiveSaveCompressedProxy::FArchiveSaveCompressedProxy(TArray<uint8>& InCompressedData, FName InCompressionFormat, ECompressionFlags InCompressionFlags)
	: CompressedData(InCompressedData)
	, CompressionFormat(InCompressionFormat)
	, CompressionFlags(InCompressionFlags)
{
	this->SetIsSaving(true);
	this->SetIsPersistent(true);
	this->SetWantBinaryPropertySerialization(true);
	bShouldSerializeToArray				= false;
	RawBytesSerialized					= 0;
	CurrentIndex						= 0;

	// Allocate temporary memory.
	TmpDataStart	= (uint8*) FMemory::Malloc(LOADING_COMPRESSION_CHUNK_SIZE);
	TmpDataEnd		= TmpDataStart + LOADING_COMPRESSION_CHUNK_SIZE;
	TmpData			= TmpDataStart;
}

/** Destructor, flushing array if needed. Also frees temporary memory. */
FArchiveSaveCompressedProxy::~FArchiveSaveCompressedProxy()
{
	// Flush is required to write out remaining tmp data to array.
	Flush();
	// Free temporary memory allocated.
	FMemory::Free( TmpDataStart );
	TmpDataStart	= NULL;
	TmpDataEnd		= NULL;
	TmpData			= NULL;
}

/**
 * Flushes tmp data to array.
 */
void FArchiveSaveCompressedProxy::Flush()
{
	if( TmpData - TmpDataStart > 0 )
	{
		// This will call Serialize so we need to indicate that we want to serialize to array.
		bShouldSerializeToArray = true;
		SerializeCompressed( TmpDataStart, TmpData - TmpDataStart, CompressionFormat, CompressionFlags);
		bShouldSerializeToArray = false;
		// Buffer is drained, reset.
		TmpData	= TmpDataStart;
	}
}

/**
 * Serializes data to archive. This function is called recursively and determines where to serialize
 * to and how to do so based on internal state.
 *
 * @param	Data	Pointer to serialize to
 * @param	Count	Number of bytes to read
 */
void FArchiveSaveCompressedProxy::Serialize( void* InData, int64 Count )
{
	uint8* SrcData = (uint8*) InData;
	// If counter > 1 it means we're calling recursively and therefore need to write to compressed data.
	if( bShouldSerializeToArray )
	{
		// Add space in array if needed and copy data there.
		int32 BytesToAdd = CurrentIndex + Count - CompressedData.Num();
		if( BytesToAdd > 0 )
		{
			CompressedData.AddUninitialized(BytesToAdd);
		}
		// Copy memory to array.
		FMemory::Memcpy( &CompressedData[CurrentIndex], SrcData, Count );
		CurrentIndex += Count;
	}
	// Regular call to serialize, queue for compression.
	else
	{
		while( Count )
		{
			int32 BytesToCopy = FMath::Min<int32>( Count, (int32)(TmpDataEnd - TmpData) );
			// Enough room in buffer to copy some data.
			if( BytesToCopy )
			{
				FMemory::Memcpy( TmpData, SrcData, BytesToCopy );
				Count -= BytesToCopy;
				TmpData += BytesToCopy;
				SrcData += BytesToCopy;
				RawBytesSerialized += BytesToCopy;
			}
			// Tmp buffer fully exhausted, compress it.
			else
			{
				// Flush existing data to array after compressing it. This will call Serialize again 
				// so we need to handle recursion.
				Flush();
			}
		}
	}
}

/**
 * Seeking is only implemented internally for writing out compressed data and asserts otherwise.
 * 
 * @param	InPos	Position to seek to
 */
void FArchiveSaveCompressedProxy::Seek( int64 InPos )
{
	// Support setting position in array.
	if( bShouldSerializeToArray )
	{
		CurrentIndex = InPos;
	}
	else
	{
		UE_LOGF(LogSerialization, Fatal,"Seeking not supported with FArchiveSaveCompressedProxy");
	}
}

/**
 * @return current position in uncompressed stream in bytes
 */
int64 FArchiveSaveCompressedProxy::Tell()
{
	// If we're serializing to array, return position in array.
	if( bShouldSerializeToArray )
	{
		return CurrentIndex;
	}
	// Return global position in raw uncompressed stream.
	else
	{
		return RawBytesSerialized;
	}
}

PRAGMA_RESTORE_UNSAFE_TYPECAST_WARNINGS
