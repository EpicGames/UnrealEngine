// Copyright Epic Games, Inc. All Rights Reserved.
#include "Serialization/ArchiveSerializedPropertyChain.h"


#if WITH_EDITORONLY_DATA
namespace UE::IDOHelpers
{
	PRAGMA_DISABLE_INTERNAL_WARNINGS
	UE_INTERNAL CORE_API TFunction<bool(TNotNull<FProperty*>)> GIsUnknownPropertyFunc;
	PRAGMA_ENABLE_INTERNAL_WARNINGS
}
#endif