// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/BitArchive.h"
#include "HAL/IConsoleManager.h"
#include "Serialization/BitWriter.h"

// Globals

CORE_API TAutoConsoleVariable<int32> CVarMaxNetStringSize(TEXT("net.MaxNetStringSize"), 16 * 1024 * 1024, TEXT("Maximum allowed size for strings sent/received by the netcode (in bytes)."));

FBitArchive::FBitArchive()
{
	ArMaxSerializeSize = CVarMaxNetStringSize.GetValueOnAnyThread();
}
