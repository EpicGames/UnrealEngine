// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/BitReader.h"

#include "CoreGlobals.h"
#include "HAL/IConsoleManager.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Math/UnrealMathUtility.h"
#include "Serialization/Archive.h"
#include "Templates/UnrealTemplate.h"
#include "Trace/Detail/Channel.h"

PRAGMA_DISABLE_UNSAFE_TYPECAST_WARNINGS

// Table.
extern const uint8 GShift[8]={0x01,0x02,0x04,0x08,0x10,0x20,0x40,0x80};
extern const uint8 GMask [8]={0x00,0x01,0x03,0x07,0x0f,0x1f,0x3f,0x7f};

namespace UE::Net
{
	static bool GLogFatalOnOverflow = false;

	static FAutoConsoleVariableRef CVarLogFatalOnOverflow(
		TEXT("BitReader.LogFatalOnOverflow"),
		GLogFatalOnOverflow,
		TEXT("LogFatal if BitReader Overflows"));

	static int32 GEnsureOnOverflow = 1;

	static FAutoConsoleVariableRef CVarEnsureOnOverflow(
		TEXT("net.BitReader.EnsureOnOverflow"),
		GEnsureOnOverflow,
		TEXT("Ensures if BitReader overflows. 'BitReader.LogFatalOnOverflow' takes precedence over this, if set."));
}

// Optimized arbitrary bit range memory copy routine.

void appBitsCpy( uint8* Dest, int32 DestBit, uint8* Src, int32 SrcBit, int32 BitCount )
{
	if( BitCount==0 ) return;

	// Special case - always at least one bit to copy,
	// a maximum of 2 bytes to read, 2 to write - only touch bytes that are actually used.
	if( BitCount <= 8 ) 
	{
		uint32 DestIndex	   = DestBit/8;
		uint32 SrcIndex	   = SrcBit /8;
		uint32 LastDest	   =( DestBit+BitCount-1 )/8;  
		uint32 LastSrc	   =( SrcBit +BitCount-1 )/8;  
		uint32 ShiftSrc     = SrcBit & 7; 
		uint32 ShiftDest    = DestBit & 7;
		uint32 FirstMask    = 0xFF << ShiftDest;  
		uint32 LastMask     = 0xFE << ((DestBit + BitCount-1) & 7) ; // Pre-shifted left by 1.	
		uint32 Accu;		

		if( SrcIndex == LastSrc )
			Accu = (Src[SrcIndex] >> ShiftSrc); 
		else
			Accu =( (Src[SrcIndex] >> ShiftSrc) | (Src[LastSrc ] << (8-ShiftSrc)) );			

		if( DestIndex == LastDest )
		{
			uint32 MultiMask = FirstMask & ~LastMask;
			Dest[DestIndex] = ( ( Dest[DestIndex] & ~MultiMask ) | ((Accu << ShiftDest) & MultiMask) );		
		}
		else
		{		
			Dest[DestIndex] = (uint8)( ( Dest[DestIndex] & ~FirstMask ) | (( Accu << ShiftDest) & FirstMask) ) ;
			Dest[LastDest ] = (uint8)( ( Dest[LastDest ] & LastMask  )  | (( Accu >> (8-ShiftDest)) & ~LastMask) ) ;
		}

		return;
	}

	// Main copier, uses byte sized shifting. Minimum size is 9 bits, so at least 2 reads and 2 writes.
	uint32 DestIndex		= DestBit/8;
	uint32 FirstSrcMask  = 0xFF << ( DestBit & 7);  
	uint32 LastDest		= ( DestBit+BitCount )/8; 
	uint32 LastSrcMask   = 0xFF << ((DestBit + BitCount) & 7); 
	uint32 SrcIndex		= SrcBit/8;
	uint32 LastSrc		= ( SrcBit+BitCount )/8;  
	int32   ShiftCount    = (DestBit & 7) - (SrcBit & 7); 
	int32   DestLoop      = LastDest-DestIndex; 
	int32   SrcLoop       = LastSrc -SrcIndex;  
	uint32 FullLoop;
	uint32 BitAccu;

	// Lead-in needs to read 1 or 2 source bytes depending on alignment.
	if( ShiftCount>=0 )
	{
		FullLoop  = FMath::Max(DestLoop, SrcLoop);  
		BitAccu   = Src[SrcIndex] << ShiftCount; 
		ShiftCount += 8; //prepare for the inner loop.
	}
	else
	{
		ShiftCount +=8; // turn shifts -7..-1 into +1..+7
		FullLoop  = FMath::Max(DestLoop, SrcLoop-1);  
		BitAccu   = Src[SrcIndex] << ShiftCount; 
		SrcIndex++;		
		ShiftCount += 8; // Prepare for inner loop.  
		BitAccu = ( ( (uint32)Src[SrcIndex] << ShiftCount ) + (BitAccu)) >> 8; 
	}

	// Lead-in - first copy.
	Dest[DestIndex] = (uint8) (( BitAccu & FirstSrcMask) | ( Dest[DestIndex] &  ~FirstSrcMask ) );
	SrcIndex++;
	DestIndex++;

	// Fast inner loop. 
	for(; FullLoop>1; FullLoop--) 
	{   // ShiftCount ranges from 8 to 15 - all reads are relevant.
		BitAccu = (( (uint32)Src[SrcIndex] << ShiftCount ) + (BitAccu)) >> 8; // Copy in the new, discard the old.
		SrcIndex++;
		Dest[DestIndex] = (uint8) BitAccu;  // Copy low 8 bits.
		DestIndex++;		
	}

	// Lead-out. 
	if( LastSrcMask != 0xFF) 
	{
		if ((uint32)(SrcBit+BitCount-1)/8 == SrcIndex ) // Last legal byte ?
		{
			BitAccu = ( ( (uint32)Src[SrcIndex] << ShiftCount ) + (BitAccu)) >> 8; 
		}
		else
		{
			BitAccu = BitAccu >> 8; 
		}		

		Dest[DestIndex] = (uint8)( ( Dest[DestIndex] & LastSrcMask ) | (BitAccu & ~LastSrcMask) );  		
	}	
}

/*-----------------------------------------------------------------------------
	FBitReader.
-----------------------------------------------------------------------------*/

//
// Reads bitstreams.
//
FBitReader::FBitReader(const uint8* Src, int64 CountBits)
	: Num(CountBits)
	, Pos(0)
{
	Buffer.AddUninitialized((CountBits + 7) >> 3);

	this->SetIsLoading(true);
	this->SetIsPersistent(true);

	// This class is exclusively used by the netcode
	ArIsNetArchive = true;

	if (Src != nullptr)
	{
		FMemory::Memcpy(Buffer.GetData(), Src, (CountBits + 7) >> 3);

		if (Num & 7)
		{
			Buffer[Num >> 3] &= GMask[Num & 7];
		}
	}
}

FBitReader::~FBitReader() = default;
FBitReader::FBitReader(const FBitReader&) = default;
FBitReader& FBitReader::operator=(const FBitReader&) = default;
FBitReader::FBitReader(FBitReader&&) = default;
FBitReader& FBitReader::operator=(FBitReader&&) = default;

void FBitReader::SetData( uint8* Src, int64 CountBits )
{
	Num			= CountBits;
	Pos			= 0;
	ClearError();

	Buffer.Empty();
	Buffer.AddUninitialized( (Num+7)>>3 );
	
	if (Src != nullptr)
	{
		FMemory::Memcpy(Buffer.GetData(), Src, (Num + 7) >> 3);

		if (Num & 7)
		{
			Buffer[Num >> 3] &= GMask[Num & 7];
		}
	}
}

void FBitReader::SetData( TArray<uint8>&& Src, int64 CountBits )
{
	Num			= CountBits;
	Pos			= 0;
	ClearError();

	Buffer = MoveTemp(Src);

	if (Num & 7)
	{
		Buffer[Num >> 3] &= GMask[Num & 7];
	}
}

void FBitReader::SetData( FBitReader& Src, int64 CountBits )
{
	Num			= CountBits;
	Pos			= 0;
	ClearError();

	// Setup network version
	SetNetVersionsFromArchive(Src);

	Buffer.Empty();
	Buffer.AddUninitialized( (CountBits+7)>>3 );
	Src.SerializeBits(Buffer.GetData(), CountBits);
}

void FBitReader::ResetData(FBitReader& Src, int64 CountBits, int64 CountBitsWithSlack)
{
	Num = CountBits;
	Pos = 0;
	ClearError();

	// Setup network version
	this->SetNetVersionsFromArchive(Src);

	Buffer.Reset((CountBitsWithSlack + 7) >> 3);
	Buffer.AddUninitialized((CountBits + 7) >> 3);
	Src.SerializeBits(Buffer.GetData(), CountBits);
}

/** This appends data from another BitReader. It checks that this bit reader is byte-aligned so it can just do a TArray::Append instead of a bitcopy.
 *	It is intended to be used by performance minded code that wants to ensure an appBitCpy is avoided.
 */
void FBitReader::AppendDataFromChecked( FBitReader& Src )
{
	check(Num % 8 == 0);
	Src.AppendTo(Buffer);
	Num += Src.GetNumBits();
}
void FBitReader::AppendDataFromChecked( uint8* Src, uint32 NumBits )
{
	check(Num % 8 == 0);

	uint32 NumBytes = (NumBits+7) >> 3;

	Buffer.AddUninitialized(NumBytes);
	FMemory::Memcpy( &Buffer[Num >> 3], Src, NumBytes );

	Num += NumBits;

	if (Num & 7)
	{
		Buffer[Num >> 3] &= GMask[Num & 7];
	}
}

void FBitReader::AppendTo( TArray<uint8> &DestBuffer )
{
	DestBuffer.Append(Buffer);
}

void FBitReader::CountMemory(FArchive& Ar) const
{
	Buffer.CountBytes(Ar);
	Ar.CountBytes(sizeof(*this), sizeof(*this));
}

void FBitReader::SetOverflowed(int64 LengthBits)
{
	using namespace UE::Net;

	if (GLogFatalOnOverflow)
	{
		UE_LOG(LogNetSerialization, Fatal, TEXT("FBitReader::SetOverflowed() called! (ReadLen: %" INT64_X_FMT ", Remaining: %" INT64_X_FMT ", Max: %" INT64_X_FMT ")"),
				LengthBits, (Num - Pos), Num);
	}
	else if (GEnsureOnOverflow)
	{
		ensureMsgf(false, TEXT("FBitReader::SetOverflowed() called! (ReadLen: %" INT64_X_FMT ", Remaining: %" INT64_X_FMT ", Max: %" INT64_X_FMT ")"),
					LengthBits, (Num - Pos), Num);
	}
	else
	{
		UE_LOG(LogNetSerialization, Error, TEXT("FBitReader::SetOverflowed() called! (ReadLen: %" INT64_X_FMT ", Remaining: %" INT64_X_FMT ", Max: %" INT64_X_FMT ")"),
				LengthBits, (Num - Pos), Num);
	}

	SetError();
}

void FBitReader::SerializeBitsWithOffset( void* Dest, int32 DestBit, int64 LengthBits )
{
	if ( IsError() || Pos+LengthBits > Num)
	{
		if (!IsError())
		{
			SetOverflowed(LengthBits);
		}
		return;
	}

	if (LengthBits != 0)
	{
		appBitsCpy((uint8*)Dest, DestBit, Buffer.GetData(), (int32)Pos, (int32)LengthBits);
		Pos += LengthBits;
	}
}

// This version is bit compatible with FArchive::SerializeIntPacked. See notes in FBitWriter::SerializeIntPacked for more info.
void FBitReader::SerializeIntPacked(uint32& OutValue)
{
	if (IsError())
	{
		OutValue = 0;
		return;
	}

	const uint8* Src = Buffer.GetData() + (Pos >> 3U);
	const uint32 BitCountUsedInByte = Pos & 7;
	const uint32 BitCountLeftInByte = 8 - (Pos & 7);
	const uint8 SrcMaskByte0 = uint8((1U << BitCountLeftInByte) - 1U);
	const uint8 SrcMaskByte1 = uint8((1U << BitCountUsedInByte) - 1U);
	const uint32 NextSrcIndex = (BitCountUsedInByte != 0);

	uint32 Value = 0;
	for (unsigned It = 0, ShiftCount = 0; It < 5; ++It, ShiftCount += 7)
	{
		if (Pos + 8 > Num)
		{
			SetOverflowed(8);
			break;
		}

		Pos += 8;

		const uint8 Byte = ((Src[0] >> BitCountUsedInByte) & SrcMaskByte0) | ((Src[NextSrcIndex] & SrcMaskByte1) << (BitCountLeftInByte & 7));
		const uint8 NextByteIndicator = Byte & 1;
		const uint32 ByteAsWord = Byte >> 1U;
		Value = (ByteAsWord << ShiftCount) | Value;
		++Src;

		if (!NextByteIndicator)
		{
			break;
		}
	}

	OutValue = Value;
}

void FBitReader::SetNetVersionsFromArchive(FArchive& Source)
{
	SetEngineNetVer(Source.EngineNetVer());
	SetGameNetVer(Source.GameNetVer());
	SetUEVer(Source.UEVer());
	SetLicenseeUEVer(Source.LicenseeUEVer());
	SetEngineVer(Source.EngineVer());
}

/*-----------------------------------------------------------------------------
	FBitReader.
-----------------------------------------------------------------------------*/

void FBitReaderMark::Copy( FBitReader& Reader, TArray<uint8> &Buffer )
{
	checkSlow(Pos<=Reader.Pos);

	int32 Bytes = (Reader.Pos - Pos + 7) >> 3;
	if( Bytes > 0 )
	{
		Buffer.SetNumUninitialized(Bytes);		// This makes room but doesnt zero
		Buffer[Bytes-1] = 0;	// Make sure the last byte is 0 out, because appBitsCpy wont touch the last bits
		appBitsCpy(Buffer.GetData(), 0, Reader.Buffer.GetData(), Pos, Reader.Pos - Pos);
	}
}

PRAGMA_RESTORE_UNSAFE_TYPECAST_WARNINGS
