// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/BitWriter.h"

#include "CoreGlobals.h"
#include "HAL/IConsoleManager.h"
#include "HAL/UnrealMemory.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Serialization/Archive.h"
#include "Trace/Detail/Channel.h"

/** CVar specifying the maximum serialization size for strings sent/received by the netcode */
extern CORE_API TAutoConsoleVariable<int32> CVarMaxNetStringSize;

PRAGMA_DISABLE_UNSAFE_TYPECAST_WARNINGS

extern const uint8 GShift[8];
extern const uint8 GMask[8];

CORE_API void appBitsCpy( uint8* Dest, int32 DestBit, uint8* Src, int32 SrcBit, int32 BitCount );

/*-----------------------------------------------------------------------------
	FBitWriter.
-----------------------------------------------------------------------------*/

/**
 * Constructor using known size the buffer needs to be
 */
FBitWriter::FBitWriter( int64 InMaxBits, bool InAllowResize /*=false*/ )
	: Num(0)
	, Max(InMaxBits)
	, bAllowOverflow(false)
{
	Buffer.AddUninitialized( (InMaxBits+7)>>3 );

	bAllowResize = InAllowResize;
	FMemory::Memzero(Buffer.GetData(), Buffer.Num());
	this->SetIsSaving(true);
	this->SetIsPersistent(true);

	// This class is exclusively used by the netcode
	ArIsNetArchive = true;
}

/**
 * Default constructor. Zeros everything.
 */
FBitWriter::FBitWriter(void)
	: Num(0)
	, Max(0)
	, bAllowResize(false)
	, bAllowOverflow(false)
{
	this->SetIsSaving(true);
	this->SetIsPersistent(true);

	// This class is exclusively used by the netcode
	ArIsNetArchive = true;
}

FBitWriter::FBitWriter(const FBitWriter&) = default;
FBitWriter& FBitWriter::operator=(const FBitWriter&) = default;
FBitWriter::FBitWriter(FBitWriter&&) = default;
FBitWriter& FBitWriter::operator=(FBitWriter&&) = default;
FBitWriter::~FBitWriter() = default;

/**
 * Resets the bit writer back to its initial state
 */
void FBitWriter::Reset(void)
{
	FArchive::Reset();
	Num = 0;
	FMemory::Memzero(Buffer.GetData(), Buffer.Num());
	this->SetIsSaving(true);
	this->SetIsPersistent(true);

	// This class is exclusively used by the netcode
	ArIsNetArchive = true;
	ArMaxSerializeSize = CVarMaxNetStringSize.GetValueOnAnyThread();
}

void FBitWriter::SerializeBits( void* Src, int64 LengthBits )
{
	if( AllowAppend(LengthBits) )
	{
		//for( int32 i=0; i<LengthBits; i++,Num++ )
		//	if( ((uint8*)Src)[i>>3] & GShift[i&7] )
		//		Buffer(Num>>3) |= GShift[Num&7];
		if( LengthBits == 1 )
		{
			if( ((uint8*)Src)[0] & 0x01 )
				Buffer[Num>>3] |= GShift[Num&7];
			Num++;
		}
		else if (Num == 0)
		{
			// Just use memcpy when setting bits directy after Reset()
			FMemory::Memcpy(Buffer.GetData(), Src, (LengthBits + 7) / 8);
			Num = LengthBits;
		}
		else
		{
			appBitsCpy(Buffer.GetData(), Num, (uint8*)Src, 0, LengthBits);
			Num += LengthBits;
		}
	}
	else
	{
		SetOverflowed(LengthBits);
	}
}
void FBitWriter::SerializeBitsWithOffset( void* Src, int32 SourceBit, int64 LengthBits )
{
	if( AllowAppend(LengthBits) )
	{
		appBitsCpy(Buffer.GetData(), Num, (uint8*)Src, SourceBit, LengthBits);
		Num += LengthBits;
	}
	else
	{
		SetOverflowed(LengthBits);
	}
}

void FBitWriter::Serialize( void* Src, int64 LengthBytes )
{
	//warning: Copied and pasted from FBitWriter::SerializeBits
	int64 LengthBits = LengthBytes*8;
	if( AllowAppend(LengthBits) )
	{
		appBitsCpy(Buffer.GetData(), Num, (uint8*)Src, 0, LengthBits);
		Num += LengthBits;
	}
	else
	{
		SetOverflowed(LengthBits);
	}
}

void FBitWriter::SerializeInt(uint32& Value, uint32 ValueMax)
{
	check(ValueMax >= 2);

	const int32 LengthBits = FMath::CeilLogTwo(ValueMax);
	uint32 WriteValue = Value;

	if (WriteValue >= ValueMax)
	{
		constexpr static const auto& Msg = TEXT("FBitWriter::SerializeInt(): Value out of bounds (Value: %u, ValueMax: %u)");

		UE_LOG(LogSerialization, Error, Msg, WriteValue, ValueMax);
		ensureMsgf(false, Msg, WriteValue, ValueMax);

		WriteValue = ValueMax - 1;
	}

	if (AllowAppend(LengthBits))
	{
		uint32 NewValue = 0;
		int64 LocalNum = Num;	// Use local var to avoid LHS

		for (uint32 Mask=1; (NewValue + Mask) < ValueMax && Mask; Mask*=2, LocalNum++)
		{
			if (WriteValue & Mask)
			{
				Buffer[LocalNum >> 3] += GShift[LocalNum & 7];
				NewValue += Mask;
			}
		}

		Num = LocalNum;
	}
	else
	{
		SetOverflowed(LengthBits);
	}
}

void FBitWriter::WriteIntWrapped(uint32 Value, uint32 ValueMax)
{
	check(ValueMax >= 2);

	const int32 LengthBits = FMath::CeilLogTwo(ValueMax);

	if (AllowAppend(LengthBits))
	{
		uint32 NewValue = 0;

		for (uint32 Mask=1; NewValue+Mask < ValueMax && Mask; Mask*=2, Num++)
		{
			if (Value & Mask)
			{
				Buffer[Num>>3] += GShift[Num&7];
				NewValue += Mask;
			}
		}
	}
	else
	{
		SetOverflowed(LengthBits);
	}
}
void FBitWriter::WriteBit( uint8 In )
{
	if( AllowAppend(1) )
	{
		if( In )
			Buffer[Num>>3] |= GShift[Num&7];
		Num++;
	}
	else
	{
		SetOverflowed(1);
	}
}

void FBitWriter::SetOverflowed(int32 LengthBits)
{
	if (!bAllowOverflow)
	{
		UE_LOG(LogNetSerialization, Error, TEXT("FBitWriter overflowed! (WriteLen: %X, Remaining: %" INT64_X_FMT ", Max: %" INT64_X_FMT ")"),
				LengthBits, (Max-Num), Max);
	}

	SetError();
}

void FBitWriter::CountMemory(FArchive& Ar) const
{
	Buffer.CountBytes(Ar);
	Ar.CountBytes(sizeof(*this), sizeof(*this));
}

/**
 * This function is bit compatible with FArchive::SerializeIntPacked. It is more efficient
 * as only a few bytes are written and the base version is best suited for writing many bytes.
 * This version can be made more efficient and take less bits when we can break backward compatibility.
 * The last byte will only need 4 bits so we're currently wasting 4 bits. Another way to pack
 * could be to store 2 bits first in order to indicate how many bytes are needed. That
 * would eliminate all shifting and masking to reconstruct the bytes. The downside is that
 * values less than 2^14 will waste 1-2 bits compared to the below algorithm.
 */
void FBitWriter::SerializeIntPacked(uint32& InValue)
{
	uint32 Value = InValue;
	uint32 BytesAsWords[5];
	uint32 ByteCount = 0;
	for (unsigned It = 0; (It == 0) | (Value != 0); ++It, Value = Value >> 7U)
	{
		const uint32 NextByteIndicator = (Value & ~0x7FU) != 0;
		const uint32 ByteAsWord = ((Value & 0x7FU) << 1U) | NextByteIndicator;
		BytesAsWords[ByteCount++] = ByteAsWord;
	}

	const int64 LengthBits = ByteCount * 8;
	if (!AllowAppend(LengthBits))
	{
		SetOverflowed(LengthBits);
		return;
	}

	const uint32 BitCountUsedInByte = Num & 7;
	const uint32 BitCountLeftInByte = 8 - (Num & 7);
	const uint8 DestMaskByte0 = uint8((1U << BitCountUsedInByte) - 1U);
	const uint8 DestMaskByte1 = 0xFFU ^ DestMaskByte0;
	const bool bStraddlesTwoBytes = (BitCountUsedInByte != 0);
	uint8* Dest = Buffer.GetData() + (Num >> 3U);

	Num += LengthBits;
	for (uint32 ByteIt = 0; ByteIt != ByteCount; ++ByteIt)
	{
		const uint32 ByteAsWord = BytesAsWords[ByteIt];

		*Dest = (*Dest & DestMaskByte0) | uint8(ByteAsWord << BitCountUsedInByte);
		++Dest;
		if (bStraddlesTwoBytes)
			*Dest = (*Dest & DestMaskByte1) | uint8(ByteAsWord >> BitCountLeftInByte);
	}
}

/*-----------------------------------------------------------------------------
	FBitWriterMark.
-----------------------------------------------------------------------------*/

void FBitWriterMark::Pop( FBitWriter& Writer )
{
	checkSlow(Num<=Writer.Num);
	checkSlow(Num<=Writer.Max);

	if( Num&7 )
	{
		Writer.Buffer[Num>>3] &= GMask[Num&7];
	}
	int32 Start = (Num       +7)>>3;
	int32 End   = (Writer.Num+7)>>3;
	if( End != Start )
	{
		checkSlow(Start<Writer.Buffer.Num());
		checkSlow(End<=Writer.Buffer.Num());
		FMemory::Memzero( &Writer.Buffer[Start], End-Start );
	}

	if (Overflowed)
	{
		Writer.SetError();
	}
	else
	{
		Writer.ClearError();
	}
	Writer.Num       = Num;
}

/** Copies the last section into a buffer. Does not clear the FBitWriter like ::Pop does */
void FBitWriterMark::Copy( FBitWriter& Writer, TArray<uint8> &Buffer )
{
	checkSlow(Num<=Writer.Num);
	checkSlow(Num<=Writer.Max);

	int32 Bytes = (Writer.Num - Num + 7) >> 3;
	if( Bytes > 0 )
	{
		Buffer.SetNumUninitialized(Bytes);		// This makes room but doesnt zero
		Buffer[Bytes-1] = 0;	// Make sure the last byte is 0 out, because appBitsCpy wont touch the last bits
		appBitsCpy(Buffer.GetData(), 0, Writer.Buffer.GetData(), Num, Writer.Num - Num);
	}
}

PRAGMA_RESTORE_UNSAFE_TYPECAST_WARNINGS
