// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/CompactBinary.h"

#include "Containers/Array.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "HAL/PlatformString.h"
#include "Hash/xxhash.h"
#include "Math/UnrealMathUtility.h"
#include "Memory/CompositeBuffer.h"
#include "Misc/CString.h"
#include "Misc/DateTime.h"
#include "Misc/Guid.h"
#include "Misc/Timespan.h"
#include "Serialization/Archive.h"
#include "Serialization/CompactBinaryValue.h"
#include "Serialization/VarInt.h"

#include <atomic>

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

namespace UE::CompactBinary::Private
{

static constexpr const uint8 GEmptyObjectValue[] = { uint8(ECbFieldType::Object), 0x00 };
static constexpr const uint8 GEmptyArrayValue[] = { uint8(ECbFieldType::Array), 0x01, 0x00 };

static std::atomic<uint32> GObjectIdRunId;
static std::atomic<uint32> GObjectIdSerial;

} // UE::CompactBinary::Private

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

void FCbFieldType::StaticAssertTypeConstants()
{
	constexpr ECbFieldType AllFlags = ECbFieldType::HasFieldName | ECbFieldType::HasFieldType;
	static_assert(!(TypeMask & AllFlags), "TypeMask is invalid!");

	static_assert(ObjectBase == ECbFieldType::Object, "ObjectBase is invalid!");
	static_assert((ObjectMask & (AllFlags | ECbFieldType::UniformObject)) == ECbFieldType::Object, "ObjectMask is invalid!");
	static_assert(!(ObjectMask & (ObjectBase ^ ECbFieldType::UniformObject)), "ObjectMask or ObjectBase is invalid!");
	static_assert(TypeMask == (ObjectMask | (ObjectBase ^ ECbFieldType::UniformObject)), "ObjectMask or ObjectBase is invalid!");

	static_assert(ArrayBase == ECbFieldType::Array, "ArrayBase is invalid!");
	static_assert((ArrayMask & (AllFlags | ECbFieldType::UniformArray)) == ECbFieldType::Array, "ArrayMask is invalid!");
	static_assert(!(ArrayMask & (ArrayBase ^ ECbFieldType::UniformArray)), "ArrayMask or ArrayBase is invalid!");
	static_assert(TypeMask == (ArrayMask | (ArrayBase ^ ECbFieldType::UniformArray)), "ArrayMask or ArrayBase is invalid!");

	static_assert(IntegerBase == ECbFieldType::IntegerPositive, "IntegerBase is invalid!");
	static_assert((IntegerMask & (AllFlags | ECbFieldType::IntegerNegative)) == ECbFieldType::IntegerPositive, "IntegerMask is invalid!");
	static_assert(!(IntegerMask & (IntegerBase ^ ECbFieldType::IntegerNegative)), "IntegerMask or IntegerBase is invalid!");
	static_assert(TypeMask == (IntegerMask | (IntegerBase ^ ECbFieldType::IntegerNegative)), "IntegerMask or IntegerBase is invalid!");

	static_assert(FloatBase == ECbFieldType::IntegerPositive, "FloatBase is invalid!");
	static_assert((FloatMask & (AllFlags | ECbFieldType::IntegerPositive)) == ECbFieldType::IntegerPositive, "FloatMask is invalid!");
	static_assert(!(FloatMask & (FloatBase ^ ECbFieldType::Float64)), "FloatMask or FloatBase is invalid!");
	static_assert(TypeMask == (FloatMask | (FloatBase ^ ECbFieldType::Float64)), "FloatMask or FloatBase is invalid!");

	static_assert(BoolBase == ECbFieldType::BoolFalse, "BoolBase is invalid!");
	static_assert((BoolMask & (AllFlags | ECbFieldType::BoolTrue)) == ECbFieldType::BoolFalse, "BoolMask is invalid!");
	static_assert(!(BoolMask & (BoolBase ^ ECbFieldType::BoolTrue)), "BoolMask or BoolBase is invalid!");
	static_assert(TypeMask == (BoolMask | (BoolBase ^ ECbFieldType::BoolTrue)), "BoolMask or BoolBase is invalid!");

	static_assert(AttachmentBase == ECbFieldType::ObjectAttachment, "AttachmentBase is invalid!");
	static_assert((AttachmentMask & (AllFlags | ECbFieldType::BinaryAttachment)) == ECbFieldType::ObjectAttachment, "AttachmentMask is invalid!");
	static_assert(!(AttachmentMask & (AttachmentBase ^ ECbFieldType::BinaryAttachment)), "AttachmentMask or AttachmentBase is invalid!");
	static_assert(TypeMask == (AttachmentMask | (AttachmentBase ^ ECbFieldType::BinaryAttachment)), "AttachmentMask or AttachmentBase is invalid!");
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FCbObjectId::FCbObjectId(const FMemoryView ObjectId)
{
	checkf(ObjectId.GetSize() == sizeof(Bytes),
		TEXT("FCbObjectId cannot be constructed from a view of %" UINT64_FMT " bytes."), ObjectId.GetSize());
	FMemory::Memcpy(Bytes, ObjectId.GetData(), sizeof(Bytes));
}

FCbObjectId FCbObjectId::NewObjectId()
{
	using namespace UE::CompactBinary::Private;

	FCbObjectId ObjectId;
	uint32* Components = reinterpret_cast<uint32*>(ObjectId.Bytes);

	const int64 Offset = 1'609'459'200; // Seconds from 1970 -> 2021
	const int64 Time = FDateTime::Now().ToUnixTimestamp() - Offset;

	uint32 RunId = GObjectIdRunId.load(std::memory_order_relaxed);
	if (RunId == 0)
	{
		// Use a GUID as a stronger source of randomness than FMath::RandRange.
		const FGuid Guid = FGuid::NewGuid();
		const uint64 Hash = FXxHash64::HashBuffer(&Guid, sizeof(FGuid)).Hash;
		uint32 ZeroSerial = 0;
		GObjectIdSerial.compare_exchange_strong(ZeroSerial, uint32(Hash), std::memory_order_relaxed);
		GObjectIdRunId.compare_exchange_strong(RunId, uint32(Hash >> 32), std::memory_order_relaxed);
	}

	Components[0] = static_cast<uint32>(Time);
	Components[1] = GObjectIdSerial.fetch_add(1, std::memory_order_relaxed);
	Components[2] = RunId;

	return ObjectId;
}

FGuid ToGuid(const FCbObjectId& Id)
{
	const FCbObjectId::ByteArray& Bytes = Id.GetBytes();
	uint32 A = static_cast<uint32>(Bytes[0]) << 24 | static_cast<uint32>(Bytes[1]) << 16 | static_cast<uint32>(Bytes[2]) << 8 | static_cast<uint32>(Bytes[3]);
	uint32 B = static_cast<uint32>(Bytes[4]) << 24 | static_cast<uint32>(Bytes[5]) << 16 | static_cast<uint32>(Bytes[6]) << 8 | static_cast<uint32>(Bytes[7]);
	uint32 C = static_cast<uint32>(Bytes[8]) << 24 | static_cast<uint32>(Bytes[9]) << 16 | static_cast<uint32>(Bytes[10]) << 8 | static_cast<uint32>(Bytes[11]);
	uint32 D = (B & 0x0000ff00u) | (C & 0xff000000u);;
	B = (B & ~0x0000f000u) | 0x00004000u;
	C = (C & ~0xc0000000u) | 0x80000000u;
	return FGuid(A, B, C, D);
}

FCbObjectId FromGuid(const FGuid& Id)
{
	uint32 Bits0 = Id.A;
	uint32 Bits1 = (Id.B & ~0x0000ff00u) | (Id.D & 0x0000ff00u);
	uint32 Bits2 = (Id.C & ~0xff000000u) | (Id.D & 0xff000000u);
	FCbObjectId::ByteArray Bytes = {
		static_cast<uint8>((Bits0 >> 24) & 0xff),
		static_cast<uint8>((Bits0 >> 16) & 0xff),
		static_cast<uint8>((Bits0 >> 8) & 0xff),
		static_cast<uint8>(Bits0 & 0xff),
		static_cast<uint8>((Bits1 >> 24) & 0xff),
		static_cast<uint8>((Bits1 >> 16) & 0xff),
		static_cast<uint8>((Bits1 >> 8) & 0xff),
		static_cast<uint8>(Bits1 & 0xff),
		static_cast<uint8>((Bits2 >> 24) & 0xff),
		static_cast<uint8>((Bits2 >> 16) & 0xff),
		static_cast<uint8>((Bits2 >> 8) & 0xff),
		static_cast<uint8>(Bits2 & 0xff)
	};
	return FCbObjectId(Bytes);
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FCbFieldView::FCbFieldView(const void* const Data, ECbFieldType Type)
{
	const uint8* Bytes = static_cast<const uint8*>(Data);
	Type = FCbFieldType::HasFieldType(Type) ? (ECbFieldType(*Bytes++) | ECbFieldType::HasFieldType) : Type;
	uint32 NameLenByteCount = 0;
	const uint64 NameLen64 = FCbFieldType::HasFieldName(Type) ? ReadVarUInt(Bytes, NameLenByteCount) : 0;
	Bytes += NameLen64 + NameLenByteCount;

	TypeWithFlags = Type;
	NameLen = uint32(FMath::Clamp<uint64>(NameLen64, 0, ~uint32(0)));
	Value = Bytes;
}

FCbObjectView FCbFieldView::AsObjectView()
{
	if (FCbFieldType::IsObject(TypeWithFlags))
	{
		Error = ECbFieldError::None;
		return FCbObjectView::FromFieldNoCheck(*this);
	}
	else
	{
		Error = ECbFieldError::TypeError;
		return FCbObjectView();
	}
}

FCbArrayView FCbFieldView::AsArrayView()
{
	if (FCbFieldType::IsArray(TypeWithFlags))
	{
		Error = ECbFieldError::None;
		return FCbArrayView::FromFieldNoCheck(*this);
	}
	else
	{
		Error = ECbFieldError::TypeError;
		return FCbArrayView();
	}
}

FMemoryView FCbFieldView::AsBinaryView(const FMemoryView Default)
{
	if (FCbValue Accessor = GetValue(); FCbFieldType::IsBinary(Accessor.GetType()))
	{
		Error = ECbFieldError::None;
		return Accessor.AsBinary();
	}
	else
	{
		Error = ECbFieldError::TypeError;
		return Default;
	}
}

FUtf8StringView FCbFieldView::AsString(const FUtf8StringView Default)
{
	if (FCbValue Accessor = GetValue(); FCbFieldType::IsString(Accessor.GetType()))
	{
		return Accessor.AsString(&Error, Default);
	}
	else
	{
		Error = ECbFieldError::TypeError;
		return Default;
	}
}

uint64 FCbFieldView::AsInteger(const uint64 Default, const UE::CompactBinary::Private::FIntegerParams Params)
{
	if (FCbValue Accessor = GetValue(); FCbFieldType::IsInteger(Accessor.GetType()))
	{
		return Accessor.AsInteger(Params, &Error, Default);
	}
	else
	{
		Error = ECbFieldError::TypeError;
		return Default;
	}
}

float FCbFieldView::AsFloat(const float Default)
{
	switch (FCbValue Accessor = GetValue(); Accessor.GetType())
	{
	case ECbFieldType::IntegerPositive:
	case ECbFieldType::IntegerNegative:
	{
		const uint64 IsNegative = uint8(Accessor.GetType()) & 1;
		constexpr uint64 OutOfRangeMask = ~((uint64(1) << /*FLT_MANT_DIG*/ 24) - 1);

		uint32 MagnitudeByteCount;
		const int64 Magnitude = ReadVarUInt(Accessor.GetData(), MagnitudeByteCount) + IsNegative;
		const uint64 IsInRange = !(Magnitude & OutOfRangeMask);
		Error = IsInRange ? ECbFieldError::None : ECbFieldError::RangeError;
		return IsInRange ? float(IsNegative ? -Magnitude : Magnitude) : Default;
	}
	case ECbFieldType::Float32:
	{
		Error = ECbFieldError::None;
		return Accessor.AsFloat32();
	}
	case ECbFieldType::Float64:
		Error = ECbFieldError::RangeError;
		return Default;
	default:
		Error = ECbFieldError::TypeError;
		return Default;
	}
}

double FCbFieldView::AsDouble(const double Default)
{
	switch (FCbValue Accessor = GetValue(); Accessor.GetType())
	{
	case ECbFieldType::IntegerPositive:
	case ECbFieldType::IntegerNegative:
	{
		const uint64 IsNegative = uint8(Accessor.GetType()) & 1;
		constexpr uint64 OutOfRangeMask = ~((uint64(1) << /*DBL_MANT_DIG*/ 53) - 1);

		uint32 MagnitudeByteCount;
		const int64 Magnitude = ReadVarUInt(Accessor.GetData(), MagnitudeByteCount) + IsNegative;
		const uint64 IsInRange = !(Magnitude & OutOfRangeMask);
		Error = IsInRange ? ECbFieldError::None : ECbFieldError::RangeError;
		return IsInRange ? double(IsNegative ? -Magnitude : Magnitude) : Default;
	}
	case ECbFieldType::Float32:
	{
		Error = ECbFieldError::None;
		return Accessor.AsFloat32();
	}
	case ECbFieldType::Float64:
	{
		Error = ECbFieldError::None;
		return Accessor.AsFloat64();
	}
	default:
		Error = ECbFieldError::TypeError;
		return Default;
	}
}

bool FCbFieldView::AsBool(const bool bDefault)
{
	FCbValue Accessor = GetValue();
	const bool bIsBool = FCbFieldType::IsBool(Accessor.GetType());
	Error = bIsBool ? ECbFieldError::None : ECbFieldError::TypeError;
	return (uint8(bIsBool) & Accessor.AsBool()) | ((!bIsBool) & bDefault); //-V564 //-V792
}

FIoHash FCbFieldView::AsObjectAttachment(const FIoHash& Default)
{
	if (FCbValue Accessor = GetValue(); FCbFieldType::IsObjectAttachment(Accessor.GetType()))
	{
		Error = ECbFieldError::None;
		return Accessor.AsObjectAttachment();
	}
	else
	{
		Error = ECbFieldError::TypeError;
		return Default;
	}
}

FIoHash FCbFieldView::AsBinaryAttachment(const FIoHash& Default)
{
	if (FCbValue Accessor = GetValue(); FCbFieldType::IsBinaryAttachment(Accessor.GetType()))
	{
		Error = ECbFieldError::None;
		return Accessor.AsBinaryAttachment();
	}
	else
	{
		Error = ECbFieldError::TypeError;
		return Default;
	}
}

FIoHash FCbFieldView::AsAttachment(const FIoHash& Default)
{
	if (FCbValue Accessor = GetValue(); FCbFieldType::IsAttachment(Accessor.GetType()))
	{
		Error = ECbFieldError::None;
		return Accessor.AsAttachment();
	}
	else
	{
		Error = ECbFieldError::TypeError;
		return Default;
	}
}

FIoHash FCbFieldView::AsHash(const FIoHash& Default)
{
	if (FCbValue Accessor = GetValue(); FCbFieldType::IsHash(Accessor.GetType()))
	{
		Error = ECbFieldError::None;
		return Accessor.AsHash();
	}
	else
	{
		Error = ECbFieldError::TypeError;
		return Default;
	}
}

FGuid FCbFieldView::AsUuid()
{
	return AsUuid(FGuid());
}

FGuid FCbFieldView::AsUuid(const FGuid& Default)
{
	if (FCbValue Accessor = GetValue(); FCbFieldType::IsUuid(Accessor.GetType()))
	{
		Error = ECbFieldError::None;
		return Accessor.AsUuid();
	}
	else
	{
		Error = ECbFieldError::TypeError;
		return Default;
	}
}

int64 FCbFieldView::AsDateTimeTicks(const int64 Default)
{
	if (FCbValue Accessor = GetValue(); FCbFieldType::IsDateTime(Accessor.GetType()))
	{
		Error = ECbFieldError::None;
		return Accessor.AsDateTimeTicks();
	}
	else
	{
		Error = ECbFieldError::TypeError;
		return Default;
	}
}

FDateTime FCbFieldView::AsDateTime()
{
	return FDateTime(AsDateTimeTicks(0));
}

FDateTime FCbFieldView::AsDateTime(FDateTime Default)
{
	return FDateTime(AsDateTimeTicks(Default.GetTicks()));
}

int64 FCbFieldView::AsTimeSpanTicks(const int64 Default)
{
	if (FCbValue Accessor = GetValue(); FCbFieldType::IsTimeSpan(Accessor.GetType()))
	{
		Error = ECbFieldError::None;
		return Accessor.AsTimeSpanTicks();
	}
	else
	{
		Error = ECbFieldError::TypeError;
		return Default;
	}
}

FTimespan FCbFieldView::AsTimeSpan()
{
	return FTimespan(AsTimeSpanTicks(0));
}

FTimespan FCbFieldView::AsTimeSpan(FTimespan Default)
{
	return FTimespan(AsTimeSpanTicks(Default.GetTicks()));
}

FCbObjectId FCbFieldView::AsObjectId(const FCbObjectId& Default)
{
	static_assert(sizeof(FCbObjectId) == 12, "FCbObjectId is expected to be 12 bytes.");
	if (FCbValue Accessor = GetValue(); FCbFieldType::IsObjectId(Accessor.GetType()))
	{
		Error = ECbFieldError::None;
		return Accessor.AsObjectId();
	}
	else
	{
		Error = ECbFieldError::TypeError;
		return Default;
	}
}

FCbCustomById FCbFieldView::AsCustomById(FCbCustomById Default)
{
	if (FCbValue Accessor = GetValue(); FCbFieldType::IsCustomById(Accessor.GetType()))
	{
		Error = ECbFieldError::None;
		return Accessor.AsCustomById();
	}
	else
	{
		Error = ECbFieldError::TypeError;
		return Default;
	}
}

FCbCustomByName FCbFieldView::AsCustomByName(FCbCustomByName Default)
{
	if (FCbValue Accessor = GetValue(); FCbFieldType::IsCustomByName(Accessor.GetType()))
	{
		Error = ECbFieldError::None;
		return Accessor.AsCustomByName();
	}
	else
	{
		Error = ECbFieldError::TypeError;
		return Default;
	}
}

FMemoryView FCbFieldView::AsCustom(uint64 Id, FMemoryView Default)
{
	FCbCustomById Custom = AsCustomById(FCbCustomById{Id, Default});
	if (Custom.Id == Id)
	{
		return Custom.Data;
	}
	else
	{
		Error = ECbFieldError::RangeError;
		return Default;
	}
}

FMemoryView FCbFieldView::AsCustom(FUtf8StringView Name, FMemoryView Default)
{
	const FCbCustomByName Custom = AsCustomByName(FCbCustomByName{Name, Default});
	if (Custom.Name.Equals(Name, ESearchCase::CaseSensitive))
	{
		return Custom.Data;
	}
	else
	{
		Error = ECbFieldError::RangeError;
		return Default;
	}
}

uint64 FCbFieldView::GetSize() const
{
	return sizeof(ECbFieldType) + GetViewNoType().GetSize();
}

uint64 FCbFieldView::GetValueSize() const
{
	switch (GetType())
	{
	case ECbFieldType::None:
	case ECbFieldType::Null:
		return 0;
	case ECbFieldType::Object:
	case ECbFieldType::UniformObject:
	case ECbFieldType::Array:
	case ECbFieldType::UniformArray:
	case ECbFieldType::Binary:
	case ECbFieldType::String:
	case ECbFieldType::CustomById:
	case ECbFieldType::CustomByName:
	{
		uint32 ValueSizeByteCount;
		const uint64 ValueSize = ReadVarUInt(Value, ValueSizeByteCount);
		return ValueSize + ValueSizeByteCount;
	}
	case ECbFieldType::IntegerPositive:
	case ECbFieldType::IntegerNegative:
		return MeasureVarUInt(Value);
	case ECbFieldType::Float32:
		return 4;
	case ECbFieldType::Float64:
		return 8;
	case ECbFieldType::BoolFalse:
	case ECbFieldType::BoolTrue:
		return 0;
	case ECbFieldType::ObjectAttachment:
	case ECbFieldType::BinaryAttachment:
	case ECbFieldType::Hash:
		return 20;
	case ECbFieldType::Uuid:
		return 16;
	case ECbFieldType::DateTime:
	case ECbFieldType::TimeSpan:
		return 8;
	case ECbFieldType::ObjectId:
		return 12;
	default:
		return 0;
	}
}

FIoHash FCbFieldView::GetHash() const
{
	FIoHashBuilder Hash;
	AppendHash(Hash);
	return Hash.Finalize();
}

void FCbFieldView::AppendHash(FIoHashBuilder& Builder) const
{
	const ECbFieldType SerializedType = FCbFieldType::GetSerializedType(TypeWithFlags);
	Builder.Update(&SerializedType, sizeof(SerializedType));
	Builder.Update(GetViewNoType());
}

bool FCbFieldView::Equals(const FCbFieldView& Other) const
{
	return FCbFieldType::GetSerializedType(TypeWithFlags) == FCbFieldType::GetSerializedType(Other.TypeWithFlags) &&
		GetViewNoType().EqualBytes(Other.GetViewNoType());
}

void FCbFieldView::CopyTo(FMutableMemoryView Buffer) const
{
	const FMemoryView Source = GetViewNoType();
	checkf(Buffer.GetSize() == sizeof(ECbFieldType) + Source.GetSize(),
		TEXT("Buffer is %" UINT64_FMT " bytes but %" UINT64_FMT " is required."),
		Buffer.GetSize(), sizeof(ECbFieldType) + Source.GetSize());
	const ECbFieldType SerializedType = FCbFieldType::GetSerializedType(TypeWithFlags);
	Buffer.CopyFrom(MakeMemoryView(&SerializedType, 1)).CopyFrom(Source);
}

void FCbFieldView::CopyTo(FArchive& Ar) const
{
	check(Ar.IsSaving());
	const FMemoryView Source = GetViewNoType();
	ECbFieldType SerializedType = FCbFieldType::GetSerializedType(TypeWithFlags);
	Ar.Serialize(&SerializedType, sizeof(SerializedType));
	Ar.Serialize(const_cast<void*>(Source.GetData()), static_cast<int64>(Source.GetSize()));
}

void FCbFieldView::IterateAttachments(FCbFieldVisitor Visitor) const
{
	switch (GetType())
	{
	case ECbFieldType::Object:
	case ECbFieldType::UniformObject:
		return FCbObjectView::FromFieldNoCheck(*this).IterateAttachments(Visitor);
	case ECbFieldType::Array:
	case ECbFieldType::UniformArray:
		return FCbArrayView::FromFieldNoCheck(*this).IterateAttachments(Visitor);
	case ECbFieldType::ObjectAttachment:
	case ECbFieldType::BinaryAttachment:
		return Visitor(*this);
	default:
		return;
	}
}

FMemoryView FCbFieldView::GetView() const
{
	const uint32 TypeSize = FCbFieldType::HasFieldType(TypeWithFlags) ? sizeof(ECbFieldType) : 0;
	const uint32 NameSize = FCbFieldType::HasFieldName(TypeWithFlags) ? NameLen + MeasureVarUInt(NameLen) : 0;
	const uint64 ValueSize = GetValueSize();
	return MakeMemoryView(static_cast<const uint8*>(Value) - TypeSize - NameSize, TypeSize + NameSize + ValueSize);
}

FMemoryView FCbFieldView::GetViewNoType() const
{
	const uint32 NameSize = FCbFieldType::HasFieldName(TypeWithFlags) ? NameLen + MeasureVarUInt(NameLen) : 0;
	const uint64 ValueSize = GetValueSize();
	return MakeMemoryView(static_cast<const uint8*>(Value) - NameSize, NameSize + ValueSize);
}

FCbFieldView FCbFieldView::operator[](FUtf8StringView Name) const
{
	switch (GetType())
	{
	case ECbFieldType::Object:
	case ECbFieldType::UniformObject:
		return FCbObjectView::FromFieldNoCheck(*this).FindView(Name);
	default:
		return FCbFieldView();
	}
}

FCbFieldViewIterator FCbFieldView::CreateViewIterator() const
{
	const ECbFieldType LocalTypeWithFlags = TypeWithFlags;
	if (FCbFieldType::HasFields(LocalTypeWithFlags))
	{
		const uint8* ValueBytes = static_cast<const uint8*>(Value);
		uint32 ValueSizeByteCount;
		const uint64 ValueSize = ReadVarUInt(ValueBytes, ValueSizeByteCount);
		ValueBytes += ValueSizeByteCount;
		const uint64 NumByteCount = FCbFieldType::IsArray(LocalTypeWithFlags) ? MeasureVarUInt(ValueBytes) : 0;
		if (ValueSize > NumByteCount)
		{
			const void* const ValueEnd = ValueBytes + ValueSize;
			ValueBytes += NumByteCount;
			const ECbFieldType UniformType = FCbFieldType::HasUniformFields(LocalTypeWithFlags) ?
				ECbFieldType(*ValueBytes++) : ECbFieldType::HasFieldType;
			return FCbFieldViewIterator::MakeRange(MakeMemoryView(ValueBytes, ValueEnd), UniformType);
		}
	}
	return FCbFieldViewIterator();
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FCbArrayView::FCbArrayView()
	: FCbFieldView(UE::CompactBinary::Private::GEmptyArrayValue)
{
}

uint64 FCbArrayView::Num() const
{
	const uint8* ValueBytes = static_cast<const uint8*>(GetValueData());
	ValueBytes += MeasureVarUInt(ValueBytes);
	uint32 NumByteCount;
	return ReadVarUInt(ValueBytes, NumByteCount);
}

uint64 FCbArrayView::GetSize() const
{
	return sizeof(ECbFieldType) + GetValueSize();
}

FIoHash FCbArrayView::GetHash() const
{
	FIoHashBuilder Hash;
	AppendHash(Hash);
	return Hash.Finalize();
}

void FCbArrayView::AppendHash(FIoHashBuilder& Builder) const
{
	const ECbFieldType SerializedType = GetType();
	Builder.Update(&SerializedType, sizeof(SerializedType));
	Builder.Update(GetValueView());
}

bool FCbArrayView::Equals(const FCbArrayView& Other) const
{
	return GetType() == Other.GetType() &&
		GetValueView().EqualBytes(Other.GetValueView());
}

void FCbArrayView::CopyTo(const FMutableMemoryView Buffer) const
{
	const FMemoryView Source = GetValueView();
	checkf(Buffer.GetSize() == sizeof(ECbFieldType) + Source.GetSize(),
		TEXT("Buffer is %" UINT64_FMT " bytes but %" UINT64_FMT " is required."),
		Buffer.GetSize(), sizeof(ECbFieldType) + Source.GetSize());
	const ECbFieldType SerializedType = GetType();
	Buffer.CopyFrom(MakeMemoryView(&SerializedType, 1)).CopyFrom(Source);
}

void FCbArrayView::CopyTo(FArchive& Ar) const
{
	check(Ar.IsSaving());
	const FMemoryView Source = GetValueView();
	ECbFieldType SerializedType = GetType();
	Ar.Serialize(&SerializedType, sizeof(SerializedType));
	Ar.Serialize(const_cast<void*>(Source.GetData()), static_cast<int64>(Source.GetSize()));
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FCbObjectView::FCbObjectView()
	: FCbFieldView(UE::CompactBinary::Private::GEmptyObjectValue)
{
}

FCbFieldView FCbObjectView::FindView(const FUtf8StringView Name) const
{
	for (const FCbFieldView& Field : *this)
	{
		if (Name.Equals(Field.GetName(), ESearchCase::CaseSensitive))
		{
			return Field;
		}
	}
	return FCbFieldView();
}

FCbFieldView FCbObjectView::FindViewIgnoreCase(const FUtf8StringView Name) const
{
	for (const FCbFieldView& Field : *this)
	{
		if (Name.Equals(Field.GetName(), ESearchCase::IgnoreCase))
		{
			return Field;
		}
	}
	return FCbFieldView();
}

FCbObjectView::operator bool() const
{
	return GetSize() > sizeof(UE::CompactBinary::Private::GEmptyObjectValue);
}

uint64 FCbObjectView::GetSize() const
{
	return sizeof(ECbFieldType) + GetValueSize();
}

FIoHash FCbObjectView::GetHash() const
{
	FIoHashBuilder Hash;
	AppendHash(Hash);
	return Hash.Finalize();
}

void FCbObjectView::AppendHash(FIoHashBuilder& Builder) const
{
	const ECbFieldType SerializedType = GetType();
	Builder.Update(&SerializedType, sizeof(SerializedType));
	Builder.Update(GetValueView());
}

bool FCbObjectView::Equals(const FCbObjectView& Other) const
{
	return GetType() == Other.GetType() &&
		GetValueView().EqualBytes(Other.GetValueView());
}

void FCbObjectView::CopyTo(const FMutableMemoryView Buffer) const
{
	const FMemoryView Source = GetValueView();
	checkf(Buffer.GetSize() == sizeof(ECbFieldType) + Source.GetSize(),
		TEXT("Buffer is %" UINT64_FMT " bytes but %" UINT64_FMT " is required."),
		Buffer.GetSize(), sizeof(ECbFieldType) + Source.GetSize());
	const ECbFieldType SerializedType = GetType();
	Buffer.CopyFrom(MakeMemoryView(&SerializedType, 1)).CopyFrom(Source);
}

void FCbObjectView::CopyTo(FArchive& Ar) const
{
	check(Ar.IsSaving());
	const FMemoryView Source = GetValueView();
	ECbFieldType SerializedType = GetType();
	Ar.Serialize(&SerializedType, sizeof(SerializedType));
	Ar.Serialize(const_cast<void*>(Source.GetData()), static_cast<int64>(Source.GetSize()));
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

template <typename FieldType>
TCbFieldIterator<FieldType>& TCbFieldIterator<FieldType>::operator++()
{
	const void* const ValueEnd = FieldType::GetValueEnd();
	const int64 AtEndMask = int64(ValueEnd == FieldsEnd) - 1;
	const ECbFieldType NextType = ECbFieldType(int64(FieldType::GetTypeWithFlags()) & AtEndMask);
	const void* const NextField = reinterpret_cast<const void*>(int64(ValueEnd) & AtEndMask);
	const void* const NextFieldsEnd = reinterpret_cast<const void*>(int64(FieldsEnd) & AtEndMask);
	FieldType::Assign(NextField, NextType);
	FieldsEnd = NextFieldsEnd;
	return *this;
}

template <typename FieldType>
uint64 TCbFieldIterator<FieldType>::GetRangeSize() const
{
	FMemoryView View;
	if (TryGetRangeView(View))
	{
		return View.GetSize();
	}
	else
	{
		uint64 Size = 0;
		for (TCbFieldIterator It(*this); It; ++It)
		{
			Size += It.GetSize();
		}
		return Size;
	}
}

template <typename FieldType>
FIoHash TCbFieldIterator<FieldType>::GetRangeHash() const
{
	FIoHashBuilder Hash;
	AppendRangeHash(Hash);
	return Hash.Finalize();
}

template <typename FieldType>
void TCbFieldIterator<FieldType>::AppendRangeHash(FIoHashBuilder& Builder) const
{
	FMemoryView View;
	if (TryGetRangeView(View))
	{
		Builder.Update(View);
	}
	else
	{
		for (TCbFieldIterator It(*this); It; ++It)
		{
			It.AppendHash(Builder);
		}
	}
}

template <typename FieldType>
void TCbFieldIterator<FieldType>::CopyRangeTo(FMutableMemoryView InBuffer) const
{
	FMemoryView Source;
	if (TryGetRangeView(Source))
	{
		checkf(InBuffer.GetSize() == Source.GetSize(),
			TEXT("Buffer is %" UINT64_FMT " bytes but %" UINT64_FMT " is required."),
			InBuffer.GetSize(), Source.GetSize());
		InBuffer.CopyFrom(Source);
	}
	else
	{
		for (TCbFieldIterator It(*this); It; ++It)
		{
			const uint64 Size = It.GetSize();
			It.CopyTo(InBuffer.Left(Size));
			InBuffer += Size;
		}
	}
}

template <typename FieldType>
void TCbFieldIterator<FieldType>::CopyRangeTo(FArchive& Ar) const
{
	check(Ar.IsSaving());
	FMemoryView Source;
	if (TryGetRangeView(Source))
	{
		Ar.Serialize(const_cast<void*>(Source.GetData()), static_cast<int64>(Source.GetSize()));
	}
	else
	{
		for (TCbFieldIterator It(*this); It; ++It)
		{
			It.CopyTo(Ar);
		}
	}
}

template <typename FieldType>
void TCbFieldIterator<FieldType>::IterateRangeAttachments(FCbFieldVisitor Visitor) const
{
	// Always iterate over non-uniform ranges because we do not know if they contain an attachment.
	if (FCbFieldType::HasFieldType(FieldType::GetTypeWithFlags()))
	{
		for (TCbFieldIterator It(*this); It; ++It)
		{
			if (FCbFieldType::MayContainAttachments(It.GetTypeWithFlags()))
			{
				It.IterateAttachments(Visitor);
			}
		}
	}
	// Only iterate over uniform ranges if the uniform type may contain an attachment.
	else
	{
		if (FCbFieldType::MayContainAttachments(FieldType::GetTypeWithFlags()))
		{
			for (TCbFieldIterator It(*this); It; ++It)
			{
				It.IterateAttachments(Visitor);
			}
		}
	}
}

template class TCbFieldIterator<FCbFieldView>;
template class TCbFieldIterator<FCbField>;

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FCbFieldIterator FCbFieldIterator::CloneRange(const FCbFieldViewIterator& It)
{
	FMemoryView View;
	if (It.TryGetRangeView(View))
	{
		return MakeRange(FSharedBuffer::Clone(View));
	}
	else
	{
		FUniqueBuffer Buffer = FUniqueBuffer::Alloc(It.GetRangeSize());
		It.CopyRangeTo(Buffer);
		return MakeRange(Buffer.MoveToShared());
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FCompositeBuffer FCbField::GetBuffer() const
{
	FMemoryView View;
	if (TryGetView(View))
	{
		return FCompositeBuffer(FSharedBuffer::MakeView(View, GetOuterBuffer()));
	}
	else
	{
		const ECbFieldType SerializedType = FCbFieldType::GetSerializedType(GetTypeWithFlags());
		return FCompositeBuffer(MakeSharedBufferFromArray(TArray<ECbFieldType, TInlineAllocator<1>>{SerializedType}),
			FSharedBuffer::MakeView(GetViewNoType(), GetOuterBuffer()));
	}
}

FCompositeBuffer FCbArray::GetBuffer() const
{
	FMemoryView View;
	if (TryGetView(View))
	{
		return FCompositeBuffer(FSharedBuffer::MakeView(View, GetOuterBuffer()));
	}
	else
	{
		return FCompositeBuffer(MakeSharedBufferFromArray(TArray<ECbFieldType, TInlineAllocator<1>>{GetType()}),
			FSharedBuffer::MakeView(GetValueView(), GetOuterBuffer()));
	}
}

FCompositeBuffer FCbObject::GetBuffer() const
{
	FMemoryView View;
	if (TryGetView(View))
	{
		return FCompositeBuffer(FSharedBuffer::MakeView(View, GetOuterBuffer()));
	}
	else
	{
		return FCompositeBuffer(MakeSharedBufferFromArray(TArray<ECbFieldType, TInlineAllocator<1>>{GetType()}),
			FSharedBuffer::MakeView(GetValueView(), GetOuterBuffer()));
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

namespace UE::CompactBinary::Private
{

PRAGMA_DISABLE_INTERNAL_WARNINGS;
FORCENOINLINE void ReferenceDebugTypes(
	FDebugCb*,
	FDebugCbObject*,
	FDebugCbUniformObject*,
	FDebugCbArray*,
	FDebugCbUniformArray*,
	FDebugCbBinary*,
	FDebugCbString*,
	FDebugCbIntegerPositive*,
	FDebugCbIntegerNegative*,
	FDebugCbFloat32*,
	FDebugCbFloat64*,
	FDebugCbObjectAttachment*,
	FDebugCbBinaryAttachment*,
	FDebugCbHash*,
	FDebugCbUuid*,
	FDebugCbDateTime*,
	FDebugCbTimeSpan*,
	FDebugCbObjectId*,
	FDebugCbCustomById*,
	FDebugCbCustomByName*)
{
}
PRAGMA_ENABLE_INTERNAL_WARNINGS;

} // UE::CompactBinary::Private

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
