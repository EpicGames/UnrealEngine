// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/CompactBinaryPackage.h"

#include "Algo/BinarySearch.h"
#include "Memory/CompositeBuffer.h"
#include "Memory/MemoryFwd.h"
#include "Memory/MemoryView.h"
#include "Serialization/Archive.h"
#include "Serialization/CompactBinarySerialization.h"
#include "Serialization/CompactBinaryWriter.h"
#include "Templates/RemoveReference.h"

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

static bool TryLoadAttachmentFromArchive(FCbAttachment& OutAttachment, FCbField&& Field, FArchive& Ar, FCbBufferAllocator Allocator)
{
	if (const FCbObjectView Object = Field.AsObjectView(); !Field.HasError())
	{
		// Empty Object
		if (!Object)
		{
			OutAttachment = FCbAttachment(FCbObject(), FCbObject().GetHash());
			return true;
		}
	}
	else if (const FIoHash ObjectHash = Field.AsObjectAttachment(); !Field.HasError())
	{
		// Object
		Field = LoadCompactBinary(Ar, Allocator);
		if (Field.IsObject())
		{
			OutAttachment = FCbAttachment(MoveTemp(Field).AsObject(), ObjectHash);
			return true;
		}
	}
	else if (const FIoHash BinaryHash = Field.AsBinaryAttachment(); !Field.HasError())
	{
		// Raw Binary
		Field = LoadCompactBinary(Ar, Allocator);
		if (FSharedBuffer Buffer = Field.AsBinary(); !Field.HasError())
		{
			OutAttachment = FCbAttachment(FCompositeBuffer(MoveTemp(Buffer)), BinaryHash);
			return true;
		}
	}
	else if (Field.IsBinary())
	{
		FSharedBuffer Buffer = MoveTemp(Field).AsBinary();
		if (Buffer.GetSize() == 0)
		{
			// Empty Raw Binary
			OutAttachment = FCbAttachment(FCompositeBuffer(MoveTemp(Buffer)));
			return true;
		}
		else if (FCompressedBuffer CompressedBuffer{FCompressedBuffer::FromCompressed(MoveTemp(Buffer))})
		{
			// Compressed Binary
			OutAttachment = FCbAttachment(MoveTemp(CompressedBuffer));
			return true;
		}
	}
	Ar.SetError();
	return false;
}

FCbAttachment::FObjectValue::FObjectValue(FCbObject&& InObject, const FIoHash* const InHash)
{
	FMemoryView View;
	if (!InObject.IsOwned() || !InObject.TryGetView(View))
	{
		Object = FCbObject::Clone(InObject);
	}
	else
	{
		Object = MoveTemp(InObject);
	}
	if (InHash)
	{
		checkSlow(*InHash == Object.GetHash());
		Hash = *InHash;
	}
	else
	{
		Hash = Object.GetHash();
	}
}

FIoHash FCbAttachment::GetHash() const
{
	if (const FCompressedBinaryValue* CompressedValue = Value.TryGet<FCompressedBinaryValue>())
	{
		return CompressedValue->Hash;
	}
	else if (const FBinaryValue* BinaryValue = Value.TryGet<FBinaryValue>())
	{
		return BinaryValue->Hash;
	}
	else if (const FObjectValue* ObjectValue = Value.TryGet<FObjectValue>())
	{
		return ObjectValue->Hash;
	}
	else
	{
		return FIoHash::Zero;
	}
}

bool FCbAttachment::TryLoad(FCbFieldIterator& Fields)
{
	if (const FCbObjectView Object = Fields.AsObjectView(); !Fields.HasError())
	{
		// Empty Object
		if (!Object)
		{
			Value.Emplace<FObjectValue>(FCbObject(), nullptr);
			++Fields;
			return true;
		}
	}
	else if (const FIoHash ObjectHash = Fields.AsObjectAttachment(); !Fields.HasError())
	{
		// Object
		++Fields;
		if (Fields.IsObject())
		{
			Value.Emplace<FObjectValue>(Fields.AsObject(), &ObjectHash);
			++Fields;
			return true;
		}
	}
	else if (const FIoHash BinaryHash = Fields.AsBinaryAttachment(); !Fields.HasError())
	{
		// Raw Binary
		++Fields;
		if (FSharedBuffer Buffer = Fields.AsBinary(); !Fields.HasError())
		{
			Value.Emplace<FBinaryValue>(MoveTemp(Buffer), BinaryHash);
			++Fields;
			return true;
		}
	}
	else if (Fields.IsBinary())
	{
		FSharedBuffer Buffer = Fields.AsBinary();
		if (Buffer.GetSize() == 0)
		{
			// Empty Raw Binary
			Value.Emplace<FBinaryValue>(MoveTemp(Buffer));
			++Fields;
			return true;
		}
		else if (FCompressedBuffer CompressedBuffer{FCompressedBuffer::FromCompressed(MoveTemp(Buffer))})
		{
			// Compressed Binary
			Value.Emplace<FCompressedBinaryValue>(MoveTemp(CompressedBuffer));
			++Fields;
			return true;
		}
	}
	return false;
}

bool FCbAttachment::TryLoad(FArchive& Ar, FCbBufferAllocator Allocator)
{
	FCbField Field = LoadCompactBinary(Ar, Allocator);
	return TryLoadAttachmentFromArchive(*this, MoveTemp(Field), Ar, Allocator);
}

void FCbAttachment::Save(FCbWriter& Writer) const
{
	if (const FObjectValue* ObjectValue = Value.TryGet<FObjectValue>())
	{
		if (ObjectValue->Object)
		{
			Writer.AddObjectAttachment(ObjectValue->Hash);
		}
		Writer.AddObject(ObjectValue->Object);
	}
	else if (const FBinaryValue* BinaryValue = Value.TryGet<FBinaryValue>())
	{
		if (BinaryValue->Buffer.GetSize() > 0)
		{
			Writer.AddBinaryAttachment(BinaryValue->Hash);
		}
		Writer.AddBinary(BinaryValue->Buffer);
	}
	else if (const FCompressedBinaryValue* CompressedValue = Value.TryGet<FCompressedBinaryValue>())
	{
		Writer.AddBinary(CompressedValue->Buffer.GetCompressed());
	}
	else
	{
		checkf(false, TEXT("Null attachments cannot be serialized."))
	}
}

void FCbAttachment::Save(FArchive& Ar) const
{
	FCbWriter Writer;
	Save(Writer);
	Writer.Save(Ar);
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

void FCbPackage::SetObject(FCbObject InObject, const FIoHash* InObjectHash, FAttachmentResolver* InResolver)
{
	if (InObject)
	{
		Object = InObject.IsOwned() ? MoveTemp(InObject) : FCbObject::Clone(InObject);
		if (InObjectHash)
		{
			ObjectHash = *InObjectHash;
			checkSlow(ObjectHash == Object.GetHash());
		}
		else
		{
			ObjectHash = Object.GetHash();
		}
		if (InResolver)
		{
			GatherAttachments(Object, *InResolver);
		}
	}
	else
	{
		Object.Reset();
		ObjectHash.Reset();
	}
}

void FCbPackage::AddAttachment(const FCbAttachment& Attachment, FAttachmentResolver* Resolver)
{
	if (!Attachment.IsNull())
	{
		const int32 Index = Algo::LowerBound(Attachments, Attachment);
		if (Attachments.IsValidIndex(Index) && Attachments[Index] == Attachment)
		{
			FCbAttachment& Existing = Attachments[Index];
			Existing = Attachment;
		}
		else
		{
			Attachments.Insert(Attachment, Index);
		}
		if (Attachment.IsObject() && Resolver)
		{
			GatherAttachments(Attachment.AsObject(), *Resolver);
		}
	}
}

int32 FCbPackage::RemoveAttachment(const FIoHash& Hash)
{
	const int32 Index = Algo::BinarySearchBy(Attachments, Hash, &FCbAttachment::GetHash);
	if (Attachments.IsValidIndex(Index))
	{
		Attachments.RemoveAt(Index);
		return 1;
	}
	return 0;
}

bool FCbPackage::Equals(const FCbPackage& Package) const
{
	return ObjectHash == Package.ObjectHash && Attachments == Package.Attachments;
}

const FCbAttachment* FCbPackage::FindAttachment(const FIoHash& Hash) const
{
	const int32 Index = Algo::BinarySearchBy(Attachments, Hash, &FCbAttachment::GetHash);
	return Attachments.IsValidIndex(Index) ? &Attachments[Index] : nullptr;
}

void FCbPackage::GatherAttachments(const FCbObject& Value, FAttachmentResolver Resolver)
{
	Value.IterateAttachments([this, &Resolver](FCbFieldView Field)
		{
			const FIoHash& Hash = Field.AsAttachment();
			if (FSharedBuffer Buffer = Resolver(Hash))
			{
				if (Field.IsObjectAttachment())
				{
					AddAttachment(FCbAttachment(FCbObject(MoveTemp(Buffer)), Hash), &Resolver);
				}
				else
				{
					AddAttachment(FCbAttachment(MoveTemp(Buffer)));
				}
			}
		});
}

bool FCbPackage::TryLoad(FCbFieldIterator& Fields)
{
	*this = FCbPackage();
	while (Fields)
	{
		if (Fields.IsNull())
		{
			++Fields;
			return true;
		}
		else if (FIoHash Hash = Fields.AsHash(); !Fields.HasError() && !Fields.IsAttachment())
		{
			++Fields;
			Object = Fields.AsObject();
			Object.MakeOwned();
			ObjectHash = Hash;
			if (Fields.HasError())
			{
				return false;
			}
			++Fields;
		}
		else
		{
			FCbAttachment Attachment;
			if (!Attachment.TryLoad(Fields))
			{
				return false;
			}
			AddAttachment(Attachment);
		}
	}
	return false;
}

bool FCbPackage::TryLoad(FArchive& Ar, FCbBufferAllocator Allocator)
{
	*this = FCbPackage();
	for (;;)
	{
		FCbField Field = LoadCompactBinary(Ar, Allocator);
		if (!Field)
		{
			Ar.SetError();
			return false;
		}

		if (Field.IsNull())
		{
			return true;
		}
		else if (FIoHash Hash = Field.AsHash(); !Field.HasError() && !Field.IsAttachment())
		{
			Field = LoadCompactBinary(Ar, Allocator);
			Object = MoveTemp(Field).AsObject();
			ObjectHash = Hash;
			if (Field.HasError())
			{
				return false;
			}
		}
		else
		{
			FCbAttachment Attachment;
			if (!TryLoadAttachmentFromArchive(Attachment, MoveTemp(Field), Ar, Allocator))
			{
				return false;
			}
			AddAttachment(Attachment);
		}
	}
}

void FCbPackage::Save(FCbWriter& Writer) const
{
	if (Object)
	{
		Writer.AddHash(ObjectHash);
		Writer.AddObject(Object);
	}
	for (const FCbAttachment& Attachment : Attachments)
	{
		Attachment.Save(Writer);
	}
	Writer.AddNull();
}

void FCbPackage::Save(FArchive& Ar) const
{
	FCbWriter Writer;
	Save(Writer);
	Writer.Save(Ar);
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
