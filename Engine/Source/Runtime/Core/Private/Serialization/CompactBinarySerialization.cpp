// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/CompactBinarySerialization.h"

#include "Containers/Array.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/SharedString.h"
#include "Containers/StringView.h"
#include "Containers/Utf8String.h"
#include "Containers/UnrealString.h"
#include "CoreGlobals.h"
#include "HAL/PlatformString.h"
#include "Logging/LogMacros.h"
#include "Logging/StructuredLog.h"
#include "Memory/MemoryView.h"
#include "Misc/AsciiSet.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Base64.h"
#include "Misc/DateTime.h"
#include "Misc/Guid.h"
#include "Misc/StringBuilder.h"
#include "Misc/Timespan.h"
#include "Serialization/Archive.h"
#include "Serialization/CompactBinaryValidation.h"
#include "Serialization/CompactBinaryValue.h"
#include "Serialization/VarInt.h"
#include "Templates/Function.h"
#include "Templates/IdentityFunctor.h"
#include "Templates/Invoke.h"
#include "Templates/RemoveReference.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/NameTypes.h"

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

namespace UE::Serialization::Private
{

template <typename CharType>
struct TCbJsonWriterConstants;

template <>
struct TCbJsonWriterConstants<ANSICHAR>
{
	static inline const FAnsiStringView LineTerminator = LINE_TERMINATOR_ANSI;
	static inline const FAnsiStringView Space = ANSITEXTVIEW(" ");
	static inline const FAnsiStringView Tab = ANSITEXTVIEW("\t");

	static inline const FAnsiStringView Null = ANSITEXTVIEW("null");
	static inline const FAnsiStringView True = ANSITEXTVIEW("true");
	static inline const FAnsiStringView False = ANSITEXTVIEW("false");

	static inline const FAnsiStringView EscapeBackslash = ANSITEXTVIEW("\\\\");
	static inline const FAnsiStringView EscapeDoubleQuote = ANSITEXTVIEW("\\\"");
	static inline const FAnsiStringView EscapeBell = ANSITEXTVIEW("\\b");
	static inline const FAnsiStringView EscapeFormFeed = ANSITEXTVIEW("\\f");
	static inline const FAnsiStringView EscapeLineFeed = ANSITEXTVIEW("\\n");
	static inline const FAnsiStringView EscapeCarriageReturn = ANSITEXTVIEW("\\r");
	static inline const FAnsiStringView EscapeTab = ANSITEXTVIEW("\\t");

	static inline const FAnsiStringView FieldId = ANSITEXTVIEW("\"Id\":");
	static inline const FAnsiStringView FieldName = ANSITEXTVIEW("\"Name\":");
	static inline const FAnsiStringView FieldData = ANSITEXTVIEW("\"Data\":");

	static inline decltype(auto) FormatEscapeUnicode4 = "\\u%04x";
	static inline decltype(auto) FormatFloat32 = "%.9g";
	static inline decltype(auto) FormatFloat64 = "%.17g";
};

template <>
struct TCbJsonWriterConstants<WIDECHAR>
{
	static inline const FWideStringView LineTerminator = LINE_TERMINATOR;
	static inline const FWideStringView Space = WIDETEXTVIEW(" ");
	static inline const FWideStringView Tab = WIDETEXTVIEW("\t");

	static inline const FWideStringView Null = WIDETEXTVIEW("null");
	static inline const FWideStringView True = WIDETEXTVIEW("true");
	static inline const FWideStringView False = WIDETEXTVIEW("false");

	static inline const FWideStringView EscapeBackslash = WIDETEXTVIEW("\\\\");
	static inline const FWideStringView EscapeDoubleQuote = WIDETEXTVIEW("\\\"");
	static inline const FWideStringView EscapeBell = WIDETEXTVIEW("\\b");
	static inline const FWideStringView EscapeFormFeed = WIDETEXTVIEW("\\f");
	static inline const FWideStringView EscapeLineFeed = WIDETEXTVIEW("\\n");
	static inline const FWideStringView EscapeCarriageReturn = WIDETEXTVIEW("\\r");
	static inline const FWideStringView EscapeTab = WIDETEXTVIEW("\\t");

	static inline const FWideStringView FieldId = WIDETEXTVIEW("\"Id\":");
	static inline const FWideStringView FieldName = WIDETEXTVIEW("\"Name\":");
	static inline const FWideStringView FieldData = WIDETEXTVIEW("\"Data\":");

	static inline decltype(auto) FormatEscapeUnicode4 = WIDETEXT("\\u%04x");
	static inline decltype(auto) FormatFloat32 = WIDETEXT("%.9g");
	static inline decltype(auto) FormatFloat64 = WIDETEXT("%.17g");
};

template <>
struct TCbJsonWriterConstants<UTF8CHAR> : TCbJsonWriterConstants<ANSICHAR>
{
};

template <typename CharType>
void AppendQuotedJsonString(TStringBuilderBase<CharType>& Builder, FUtf8StringView Value)
{
	using FConstants = TCbJsonWriterConstants<CharType>;
	const FAsciiSet EscapeSet("\\\"\b\f\n\r\t"
		"\x00\x01\x02\x03\x04\x05\x06\x07\x08\x09\x0a\x0b\x0c\x0d\x0e\x0f"
		"\x10\x11\x12\x13\x14\x15\x16\x17\x18\x19\x1a\x1b\x1c\x1d\x1e\x1f");
	Builder << '\"';
	while (!Value.IsEmpty())
	{
		FUtf8StringView Verbatim = FAsciiSet::FindPrefixWithout(Value, EscapeSet);
		Builder << Verbatim;
		Value.RightChopInline(Verbatim.Len());
		FUtf8StringView Escape = FAsciiSet::FindPrefixWith(Value, EscapeSet);
		for (UTF8CHAR Char : Escape)
		{
			switch (Char)
			{
			case UTF8CHAR('\\'): Builder << FConstants::EscapeBackslash; break;
			case UTF8CHAR('\"'): Builder << FConstants::EscapeDoubleQuote; break;
			case UTF8CHAR('\b'): Builder << FConstants::EscapeBell; break;
			case UTF8CHAR('\f'): Builder << FConstants::EscapeFormFeed; break;
			case UTF8CHAR('\n'): Builder << FConstants::EscapeLineFeed; break;
			case UTF8CHAR('\r'): Builder << FConstants::EscapeCarriageReturn; break;
			case UTF8CHAR('\t'): Builder << FConstants::EscapeTab; break;
			default:
				Builder.Appendf(FConstants::FormatEscapeUnicode4, uint32(Char));
				break;
			}
		}
		Value.RightChopInline(Escape.Len());
	}
	Builder << '\"';
}

template void AppendQuotedJsonString<UTF8CHAR>(FUtf8StringBuilderBase& Builder, FUtf8StringView Value);
template void AppendQuotedJsonString<WIDECHAR>(FWideStringBuilderBase& Builder, FUtf8StringView Value);

} // UE::Serialization::Private

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

namespace UE::Serialization::Private
{

void LogFieldTooLargeForArrayWarning(uint64 ArraySize)
{
	UE_LOGFMT(LogSerialization, Warning, "Attempting to load a FCbFieldView that has too many elements ({ArraySize}) to fit in TArray.", ArraySize);
}

} // namespace UE::Serialization::Private

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

uint64 MeasureCompactBinary(FMemoryView View, ECbFieldType Type)
{
	uint64 Size;
	return TryMeasureCompactBinary(View, Type, Size, Type) ? Size : 0;
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

bool TryMeasureCompactBinary(FMemoryView View, ECbFieldType& OutType, uint64& OutSize, ECbFieldType Type)
{
	uint64 Size = 0;

	if (FCbFieldType::HasFieldType(Type))
	{
		if (View.GetSize() == 0)
		{
			OutType = ECbFieldType::None;
			OutSize = 1;
			return false;
		}

		Type = *static_cast<const ECbFieldType*>(View.GetData());
		View += 1;
		Size += 1;
	}

	bool bDynamicSize = false;
	uint64 FixedSize = 0;
	switch (FCbFieldType::GetType(Type))
	{
	case ECbFieldType::Null:
		break;
	case ECbFieldType::Object:
	case ECbFieldType::UniformObject:
	case ECbFieldType::Array:
	case ECbFieldType::UniformArray:
	case ECbFieldType::Binary:
	case ECbFieldType::String:
	case ECbFieldType::IntegerPositive:
	case ECbFieldType::IntegerNegative:
	case ECbFieldType::CustomById:
	case ECbFieldType::CustomByName:
		bDynamicSize = true;
		break;
	case ECbFieldType::Float32:
		FixedSize = 4;
		break;
	case ECbFieldType::Float64:
		FixedSize = 8;
		break;
	case ECbFieldType::BoolFalse:
	case ECbFieldType::BoolTrue:
		break;
	case ECbFieldType::ObjectAttachment:
	case ECbFieldType::BinaryAttachment:
	case ECbFieldType::Hash:
		FixedSize = 20;
		break;
	case ECbFieldType::Uuid:
		FixedSize = 16;
		break;
	case ECbFieldType::DateTime:
	case ECbFieldType::TimeSpan:
		FixedSize = 8;
		break;
	case ECbFieldType::ObjectId:
		FixedSize = 12;
		break;
	case ECbFieldType::None:
	default:
		OutType = ECbFieldType::None;
		OutSize = 0;
		return false;
	}

	OutType = Type;

	if (FCbFieldType::HasFieldName(Type))
	{
		if (View.GetSize() == 0)
		{
			OutSize = Size + 1;
			return false;
		}

		uint32 NameLenByteCount = MeasureVarUInt(View.GetData());
		if (View.GetSize() < NameLenByteCount)
		{
			OutSize = Size + NameLenByteCount;
			return false;
		}

		const uint64 NameLen = ReadVarUInt(View.GetData(), NameLenByteCount);
		const uint64 NameSize = NameLen + NameLenByteCount;

		if (bDynamicSize && View.GetSize() < NameSize)
		{
			OutSize = Size + NameSize;
			return false;
		}

		View += NameSize;
		Size += NameSize;
	}

	switch (FCbFieldType::GetType(Type))
	{
	case ECbFieldType::Object:
	case ECbFieldType::UniformObject:
	case ECbFieldType::Array:
	case ECbFieldType::UniformArray:
	case ECbFieldType::Binary:
	case ECbFieldType::String:
	case ECbFieldType::CustomById:
	case ECbFieldType::CustomByName:
		if (View.GetSize() == 0)
		{
			OutSize = Size + 1;
			return false;
		}
		else
		{
			uint32 ValueSizeByteCount = MeasureVarUInt(View.GetData());
			if (View.GetSize() < ValueSizeByteCount)
			{
				OutSize = Size + ValueSizeByteCount;
				return false;
			}
			const uint64 ValueSize = ReadVarUInt(View.GetData(), ValueSizeByteCount);
			OutSize = Size + ValueSize + ValueSizeByteCount;
			return true;
		}
	case ECbFieldType::IntegerPositive:
	case ECbFieldType::IntegerNegative:
		if (View.GetSize() == 0)
		{
			OutSize = Size + 1;
			return false;
		}
		OutSize = Size + MeasureVarUInt(View.GetData());
		return true;
	default:
		OutSize = Size + FixedSize;
		return true;
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FCbField LoadCompactBinary(FArchive& Ar, FCbBufferAllocator Allocator)
{
	return LoadAndValidateCompactBinary(Ar, Allocator, ECbValidateMode::Default);
}

FCbField LoadAndValidateCompactBinary(FArchive& Ar, FCbBufferAllocator Allocator, ECbValidateMode ValidateMode)
{
	TArray<uint8, TInlineAllocator<64>> HeaderBytes;
	ECbFieldType FieldType;
	uint64 FieldSize = 1;

	for (const int64 StartPos = Ar.Tell(); !Ar.IsError() && FieldSize > 0;)
	{
		// Read in small increments until the total field size is known, to avoid reading too far.
		const int32 ReadSize = int32(FieldSize - HeaderBytes.Num());
		const int32 ReadOffset = HeaderBytes.AddUninitialized(ReadSize);
		Ar.Serialize(HeaderBytes.GetData() + ReadOffset, ReadSize);

		if (!Ar.IsError() && TryMeasureCompactBinary(MakeMemoryView(HeaderBytes), FieldType, FieldSize))
		{
			if (FieldSize <= uint64(Ar.TotalSize() - StartPos))
			{
				FUniqueBuffer Buffer = Allocator(FieldSize);
				checkf(Buffer.GetSize() == FieldSize, TEXT("Allocator returned a buffer of size %" UINT64_FMT " bytes "
					"when %" UINT64_FMT " bytes were requested."), Buffer.GetSize(), FieldSize);

				FMutableMemoryView TailView = Buffer.GetView().CopyFrom(MakeMemoryView(HeaderBytes));
				if (!TailView.IsEmpty())
				{
					// Read the remainder of the field.
					Ar.Serialize(TailView.GetData(), static_cast<int64>(TailView.GetSize()));
				}

				if (!Ar.IsError() && (ValidateMode == ECbValidateMode::None || ValidateCompactBinary(Buffer, ValidateMode) == ECbValidateError::None))
				{
					return FCbField(Buffer.MoveToShared());
				}
			}
			break;
		}
	}

	Ar.SetError();
	return FCbField();
}

FCbFieldView LoadCompactBinaryView(FArchive& Ar, FCbMemoryAllocator Allocator)
{
	return LoadAndValidateCompactBinaryView(Ar, Allocator, ECbValidateMode::Default);
}

FCbFieldView LoadAndValidateCompactBinaryView(FArchive& Ar, FCbMemoryAllocator Allocator, ECbValidateMode ValidateMode)
{
	TArray<uint8, TInlineAllocator<64>> HeaderBytes;
	ECbFieldType FieldType;
	uint64 FieldSize = 1;

	for (const int64 StartPos = Ar.Tell(); !Ar.IsError() && FieldSize > 0;)
	{
		// Read in small increments until the total field size is known, to avoid reading too far.
		const int32 ReadSize = int32(FieldSize - HeaderBytes.Num());
		const int32 ReadOffset = HeaderBytes.AddUninitialized(ReadSize);
		Ar.Serialize(HeaderBytes.GetData() + ReadOffset, ReadSize);

		if (!Ar.IsError() && TryMeasureCompactBinary(MakeMemoryView(HeaderBytes), FieldType, FieldSize))
		{
			if (FieldSize <= uint64(Ar.TotalSize() - StartPos))
			{
				FMutableMemoryView View = Allocator(FieldSize);
				checkf(View.GetSize() == FieldSize, TEXT("Allocator returned a buffer of size %" UINT64_FMT " bytes "
					"when %" UINT64_FMT " bytes were requested."), View.GetSize(), FieldSize);

				FMutableMemoryView TailView = View.CopyFrom(MakeMemoryView(HeaderBytes));
				if (!TailView.IsEmpty())
				{
					// Read the remainder of the field.
					Ar.Serialize(TailView.GetData(), static_cast<int64>(TailView.GetSize()));
				}

				if (!Ar.IsError() && (ValidateMode == ECbValidateMode::None || ValidateCompactBinary(View, ValidateMode) == ECbValidateError::None))
				{
					return FCbFieldView(View.GetData());
				}
			}
			break;
		}
	}

	Ar.SetError();
	return FCbFieldView();
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

void SaveCompactBinary(FArchive& Ar, const FCbFieldView& Field)
{
	check(Ar.IsSaving());
	Field.CopyTo(Ar);
}

void SaveCompactBinary(FArchive& Ar, const FCbArrayView& Array)
{
	check(Ar.IsSaving());
	Array.CopyTo(Ar);
}

void SaveCompactBinary(FArchive& Ar, const FCbObjectView& Object)
{
	check(Ar.IsSaving());
	Object.CopyTo(Ar);
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

template <typename T, typename ConvertType>
static FArchive& SerializeCompactBinary(FArchive& Ar, T& Value, ConvertType&& Convert)
{
	if (Ar.IsLoading())
	{
		Value = Invoke(Forward<ConvertType>(Convert), LoadCompactBinary(Ar));
	}
	else if (Ar.IsSaving())
	{
		Value.CopyTo(Ar);
	}
	else
	{
		checkNoEntry();
	}
	return Ar;
}

FArchive& operator<<(FArchive& Ar, FCbField& Field)
{
	return SerializeCompactBinary(Ar, Field, FIdentityFunctor());
}

FArchive& operator<<(FArchive& Ar, FCbArray& Array)
{
	return SerializeCompactBinary(Ar, Array, [](FCbField&& Field) { return MoveTemp(Field).AsArray(); });
}

FArchive& operator<<(FArchive& Ar, FCbObject& Object)
{
	return SerializeCompactBinary(Ar, Object, [](FCbField&& Field) { return MoveTemp(Field).AsObject(); });
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

bool LoadFromCompactBinary(FCbFieldView Field, FUtf8StringBuilderBase& OutValue)
{
	OutValue << Field.AsString();
	return !Field.HasError();
}

bool LoadFromCompactBinary(FCbFieldView Field, FWideStringBuilderBase& OutValue)
{
	OutValue << Field.AsString();
	return !Field.HasError();
}

bool LoadFromCompactBinary(FCbFieldView Field, UE::FUtf8SharedString& OutString)
{
	OutString = Field.AsString();
	return !Field.HasError();
}

bool LoadFromCompactBinary(FCbFieldView Field, UE::FWideSharedString& OutString)
{
	TWideStringBuilder<512> String;
	if (LoadFromCompactBinary(Field, String))
	{
		OutString = String;
		return true;
	}
	OutString.Reset();
	return false;
}

bool LoadFromCompactBinary(FCbFieldView Field, FUtf8String& OutValue)
{
	OutValue = Field.AsString();
	return !Field.HasError();
}

bool LoadFromCompactBinary(FCbFieldView Field, FWideString& OutValue)
{
	OutValue = FWideString(Field.AsString());
	return !Field.HasError();
}

bool LoadFromCompactBinary(FCbFieldView Field, FName& OutValue)
{
	OutValue = FName(Field.AsString());
	return !Field.HasError();
}

bool LoadFromCompactBinary(FCbFieldView Field, FGuid& OutValue)
{
	OutValue = Field.AsUuid();
	return !Field.HasError();
}

bool LoadFromCompactBinary(FCbFieldView Field, FGuid& OutValue, const FGuid& Default)
{
	OutValue = Field.AsUuid(Default);
	return !Field.HasError();
}

bool LoadFromCompactBinary(FCbFieldView Field, FDateTime& OutValue)
{
	OutValue = Field.AsDateTime();
	return !Field.HasError();
}

bool LoadFromCompactBinary(FCbFieldView Field, FDateTime& OutValue, FDateTime Default)
{
	OutValue = Field.AsDateTime(Default);
	return !Field.HasError();
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

template <typename CharType>
class TCbJsonWriter
{
	using FConstants = UE::Serialization::Private::TCbJsonWriterConstants<CharType>;

public:
	explicit TCbJsonWriter(
		TStringBuilderBase<CharType>& InBuilder,
		TStringView<CharType> InLineTerminator = FConstants::LineTerminator,
		TStringView<CharType> InIndent = FConstants::Tab,
		TStringView<CharType> InSpace = FConstants::Space)
		: Builder(InBuilder)
		, Indent(InIndent)
		, Space(InSpace)
	{
		NewLineAndIndent << InLineTerminator;
	}

	void WriteField(FCbFieldView Field)
	{
		WriteOptionalComma();
		WriteOptionalNewLine();

		if (FUtf8StringView Name = Field.GetName(); !Name.IsEmpty())
		{
			AppendQuotedString(Name);
			Builder << ':' << Space;
		}

		switch (FCbValue Accessor = Field.GetValue(); Accessor.GetType())
		{
		case ECbFieldType::Null:
			Builder << FConstants::Null;
			break;
		case ECbFieldType::Object:
		case ECbFieldType::UniformObject:
			Builder << '{';
			NewLineAndIndent << Indent;
			bNeedsNewLine = true;
			for (FCbFieldView It : Field)
			{
				WriteField(It);
			}
			NewLineAndIndent.RemoveSuffix(Indent.Len());
			if (bNeedsComma)
			{
				WriteOptionalNewLine();
			}
			Builder << '}';
			break;
		case ECbFieldType::Array:
		case ECbFieldType::UniformArray:
			Builder << '[';
			NewLineAndIndent << Indent;
			bNeedsNewLine = true;
			for (FCbFieldView It : Field)
			{
				WriteField(It);
			}
			NewLineAndIndent.RemoveSuffix(Indent.Len());
			if (bNeedsComma)
			{
				WriteOptionalNewLine();
			}
			Builder << ']';
			break;
		case ECbFieldType::Binary:
			AppendBase64String(Accessor.AsBinary());
			break;
		case ECbFieldType::String:
			AppendQuotedString(Accessor.AsString());
			break;
		case ECbFieldType::IntegerPositive:
			Builder << Accessor.AsIntegerPositive();
			break;
		case ECbFieldType::IntegerNegative:
			Builder << Accessor.AsIntegerNegative();
			break;
		case ECbFieldType::Float32:
			Builder.Appendf(FConstants::FormatFloat32, Accessor.AsFloat32());
			break;
		case ECbFieldType::Float64:
			Builder.Appendf(FConstants::FormatFloat64, Accessor.AsFloat64());
			break;
		case ECbFieldType::BoolFalse:
			Builder << FConstants::False;
			break;
		case ECbFieldType::BoolTrue:
			Builder << FConstants::True;
			break;
		case ECbFieldType::ObjectAttachment:
		case ECbFieldType::BinaryAttachment:
			Builder << '"' << Accessor.AsAttachment() << '"';
			break;
		case ECbFieldType::Hash:
			Builder << '"' << Accessor.AsHash() << '"';
			break;
		case ECbFieldType::Uuid:
			Builder << '"' << Accessor.AsUuid() << '"';
			break;
		case ECbFieldType::DateTime:
			Builder << '"' << FDateTime(Accessor.AsDateTimeTicks()).ToIso8601() << '"';
			break;
		case ECbFieldType::TimeSpan:
		{
			const FTimespan Span(Accessor.AsTimeSpanTicks());
			if (Span.GetDays() == 0)
			{
				Builder << '"' << Span.ToString(TEXT("%h:%m:%s.%n")) << '"';
			}
			else
			{
				Builder << '"' << Span.ToString(TEXT("%d.%h:%m:%s.%n")) << '"';
			}
			break;
		}
		case ECbFieldType::ObjectId:
			Builder << '"' << Accessor.AsObjectId() << '"';
			break;
		case ECbFieldType::CustomById:
		{
			FCbCustomById Custom = Accessor.AsCustomById();
			Builder << '{' << Space << FConstants::FieldId << Space;
			Builder << Custom.Id;
			Builder << ',' << Space << FConstants::FieldData << Space;
			AppendBase64String(Custom.Data);
			Builder << Space << '}';
			break;
		}
		case ECbFieldType::CustomByName:
		{
			FCbCustomByName Custom = Accessor.AsCustomByName();
			Builder << '{' << Space << FConstants::FieldName << Space;
			AppendQuotedString(Custom.Name);
			Builder << ',' << Space << FConstants::FieldData << Space;
			AppendBase64String(Custom.Data);
			Builder << Space << '}';
			break;
		}
		default:
			checkNoEntry();
			break;
		}

		bNeedsComma = true;
		bNeedsNewLine = true;
	}

private:
	void WriteOptionalComma()
	{
		if (bNeedsComma)
		{
			bNeedsComma = false;
			Builder << ',';
		}
	}

	void WriteOptionalNewLine()
	{
		if (bNeedsNewLine)
		{
			bNeedsNewLine = false;
			Builder << NewLineAndIndent;
		}
	}

	void AppendQuotedString(FUtf8StringView Value)
	{
		UE::Serialization::Private::AppendQuotedJsonString(Builder, Value);
	}

	void AppendBase64String(FMemoryView Value)
	{
		checkf(Value.GetSize() <= 512 * 1024 * 1024,
			TEXT("Encoding 512 MiB or larger is not supported. Size: %" UINT64_FMT), Value.GetSize());
		const uint32 EncodedSize = FBase64::GetEncodedDataSize(uint32(Value.GetSize()));
		// Encode writes a zero-terminator after its text, so we need to have allocated space for it to write to by
		// reserving at least one more than the EncodedSize. We handle that by reserving space for the trailing quote
		// symbol we write after it; Encode will write 0 to it and we then overwrite it with a quote.
		Builder.Reserve(Builder.Len() + 2 + EncodedSize);
		Builder << '"';
		const int32 EncodedIndex = Builder.AddUninitialized(int32(EncodedSize));
		FBase64::Encode(static_cast<const uint8*>(Value.GetData()), uint32(Value.GetSize()),
			reinterpret_cast<CharType*>(Builder.GetData() + EncodedIndex));
		Builder << '"';
	}

private:
	TStringBuilderBase<CharType>& Builder;
	TStringBuilderWithBuffer<CharType, 32> NewLineAndIndent;
	TStringView<CharType> Indent;
	TStringView<CharType> Space;
	bool bNeedsComma{false};
	bool bNeedsNewLine{false};
};

void CompactBinaryToJson(const FCbFieldView& Field, FUtf8StringBuilderBase& Builder)
{
	TCbJsonWriter<UTF8CHAR> Writer(Builder);
	Writer.WriteField(Field);
}

void CompactBinaryToJson(const FCbFieldView& Field, FWideStringBuilderBase& Builder)
{
	TCbJsonWriter<WIDECHAR> Writer(Builder);
	Writer.WriteField(Field);
}

void CompactBinaryToJson(const FCbArrayView& Array, FUtf8StringBuilderBase& Builder)
{
	CompactBinaryToJson(Array.AsFieldView(), Builder);
}

void CompactBinaryToJson(const FCbArrayView& Array, FWideStringBuilderBase& Builder)
{
	CompactBinaryToJson(Array.AsFieldView(), Builder);
}

void CompactBinaryToJson(const FCbObjectView& Object, FUtf8StringBuilderBase& Builder)
{
	CompactBinaryToJson(Object.AsFieldView(), Builder);
}

void CompactBinaryToJson(const FCbObjectView& Object, FWideStringBuilderBase& Builder)
{
	CompactBinaryToJson(Object.AsFieldView(), Builder);
}

void CompactBinaryToCompactJson(const FCbFieldView& Field, FUtf8StringBuilderBase& Builder)
{
	TCbJsonWriter<UTF8CHAR> Writer(Builder, ANSITEXTVIEW(""), ANSITEXTVIEW(""), ANSITEXTVIEW(""));
	Writer.WriteField(Field);
}

void CompactBinaryToCompactJson(const FCbFieldView& Field, FWideStringBuilderBase& Builder)
{
	TCbJsonWriter<WIDECHAR> Writer(Builder, WIDETEXTVIEW(""), WIDETEXTVIEW(""), WIDETEXTVIEW(""));
	Writer.WriteField(Field);
}

void CompactBinaryToCompactJson(const FCbArrayView& Array, FUtf8StringBuilderBase& Builder)
{
	CompactBinaryToCompactJson(Array.AsFieldView(), Builder);
}

void CompactBinaryToCompactJson(const FCbArrayView& Array, FWideStringBuilderBase& Builder)
{
	CompactBinaryToCompactJson(Array.AsFieldView(), Builder);
}

void CompactBinaryToCompactJson(const FCbObjectView& Object, FUtf8StringBuilderBase& Builder)
{
	CompactBinaryToCompactJson(Object.AsFieldView(), Builder);
}

void CompactBinaryToCompactJson(const FCbObjectView& Object, FWideStringBuilderBase& Builder)
{
	CompactBinaryToCompactJson(Object.AsFieldView(), Builder);
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
