// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/CompactBinaryValidation.h"

#include "Algo/Sort.h"
#include "Compression/CompressedBuffer.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/ContainersFwd.h"
#include "Containers/StringFwd.h"
#include "Containers/StringView.h"
#include "HAL/PlatformMemory.h"
#include "IO/IoHash.h"
#include "Memory/MemoryView.h"
#include "Memory/SharedBuffer.h"
#include "Misc/ByteSwap.h"
#include "Serialization/CompactBinary.h"
#include "Serialization/CompactBinaryPackage.h"
#include "Serialization/VarInt.h"

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * Adds the given error(s) to the error mask.
 *
 * This function exists to make validation errors easier to debug by providing one location to set a breakpoint.
 */
FORCENOINLINE static void AddError(ECbValidateError& OutError, const ECbValidateError InError)
{
	OutError |= InError;
}

/**
 * Validate and read a field type from the view.
 *
 * A type argument with the HasFieldType flag indicates that the type will not be read from the view.
 */
static ECbFieldType ValidateCbFieldType(FMemoryView& View, ECbValidateMode Mode, ECbValidateError& Error, ECbFieldType Type = ECbFieldType::HasFieldType)
{
	if (FCbFieldType::HasFieldType(Type))
	{
		if (View.GetSize() >= sizeof(ECbFieldType))
		{
			Type = *static_cast<const ECbFieldType*>(View.GetData());
			View += sizeof(ECbFieldType);
			if (FCbFieldType::HasFieldType(Type))
			{
				AddError(Error, ECbValidateError::InvalidType);
			}
		}
		else
		{
			AddError(Error, ECbValidateError::OutOfBounds);
			View.Reset();
			return ECbFieldType::None;
		}
	}

	if (FCbFieldType::GetSerializedType(Type) != Type)
	{
		AddError(Error, ECbValidateError::InvalidType);
		View.Reset();
	}

	return Type;
}

/**
 * Validate and read an unsigned integer from the view.
 *
 * Modifies the view to start at the end of the value, and adds error flags if applicable.
 */
static uint64 ValidateCbUInt(FMemoryView& View, ECbValidateMode Mode, ECbValidateError& Error)
{
	if (View.GetSize() > 0 && View.GetSize() >= MeasureVarUInt(View.GetData()))
	{
		uint32 ValueByteCount;
		const uint64 Value = ReadVarUInt(View.GetData(), ValueByteCount);
		if (EnumHasAnyFlags(Mode, ECbValidateMode::Format) && ValueByteCount > MeasureVarUInt(Value))
		{
			AddError(Error, ECbValidateError::InvalidInteger);
		}
		View += ValueByteCount;
		return Value;
	}
	else
	{
		AddError(Error, ECbValidateError::OutOfBounds);
		View.Reset();
		return 0;
	}
}

/**
 * Validate a 64-bit floating point value from the view.
 *
 * Modifies the view to start at the end of the value, and adds error flags if applicable.
 */
static void ValidateCbFloat64(FMemoryView& View, ECbValidateMode Mode, ECbValidateError& Error)
{
	if (View.GetSize() >= sizeof(double))
	{
		if (EnumHasAnyFlags(Mode, ECbValidateMode::Format))
		{
			const uint64 RawValue = NETWORK_ORDER64(FPlatformMemory::ReadUnaligned<uint64>(View.GetData()));
			const double Value = reinterpret_cast<const double&>(RawValue);
			if (Value == double(float(Value)))
			{
				AddError(Error, ECbValidateError::InvalidFloat);
			}
		}
		View += sizeof(double);
	}
	else
	{
		AddError(Error, ECbValidateError::OutOfBounds);
		View.Reset();
	}
}

/**
 * Validate and read a fixed-size value from the view.
 *
 * Modifies the view to start at the end of the value, and adds error flags if applicable.
 */
static FMemoryView ValidateCbFixedValue(FMemoryView& View, ECbValidateMode Mode, ECbValidateError& Error, uint64 Size)
{
	const FMemoryView Value = View.Left(Size);
	View += Size;
	if (Value.GetSize() < Size)
	{
		AddError(Error, ECbValidateError::OutOfBounds);
	}
	return Value;
};

/**
 * Validate and read a value from the view where the view begins with the value size.
 *
 * Modifies the view to start at the end of the value, and adds error flags if applicable.
 */
static FMemoryView ValidateCbDynamicValue(FMemoryView& View, ECbValidateMode Mode, ECbValidateError& Error)
{
	const uint64 ValueSize = ValidateCbUInt(View, Mode, Error);
	return ValidateCbFixedValue(View, Mode, Error, ValueSize);
}

/**
 * Validate and read a string from the view.
 *
 * Modifies the view to start at the end of the string, and adds error flags if applicable.
 */
static FUtf8StringView ValidateCbString(FMemoryView& View, ECbValidateMode Mode, ECbValidateError& Error)
{
	const FMemoryView Value = ValidateCbDynamicValue(View, Mode, Error);
	return FUtf8StringView(static_cast<const UTF8CHAR*>(Value.GetData()), static_cast<int32>(Value.GetSize()));
}

static FCbFieldView ValidateCbField(FMemoryView& View, ECbValidateMode Mode, ECbValidateError& Error, ECbFieldType ExternalType);

/** A type that checks whether all validated fields are of the same type. */
class FCbUniformFieldsValidator
{
public:
	inline explicit FCbUniformFieldsValidator(ECbFieldType InExternalType)
		: ExternalType(InExternalType)
	{
	}

	inline FCbFieldView ValidateField(FMemoryView& View, ECbValidateMode Mode, ECbValidateError& Error)
	{
		const void* const FieldData = View.GetData();
		if (FCbFieldView Field = ValidateCbField(View, Mode, Error, ExternalType))
		{
			++FieldCount;
			if (FCbFieldType::HasFieldType(ExternalType))
			{
				const ECbFieldType FieldType = *static_cast<const ECbFieldType*>(FieldData);
				if (FieldCount == 1)
				{
					FirstType = FieldType;
				}
				else if (FieldType != FirstType)
				{
					bUniform = false;
				}
			}
			return Field;
		}

		// It may not safe to check for uniformity if the field was invalid.
		bUniform = false;
		return FCbFieldView();
	}

	inline bool IsUniform() const { return FieldCount > 0 && bUniform; }

private:
	uint32 FieldCount = 0;
	bool bUniform = true;
	ECbFieldType FirstType = ECbFieldType::None;
	ECbFieldType ExternalType;
};

static void ValidateCbObject(FMemoryView& View, ECbValidateMode Mode, ECbValidateError& Error, ECbFieldType ObjectType)
{
	const uint64 Size = ValidateCbUInt(View, Mode, Error);
	FMemoryView ObjectView = View.Left(Size);
	View += Size;

	if (Size > 0)
	{
		TArray<FUtf8StringView, TInlineAllocator<16>> Names;

		const bool bUniformObject = FCbFieldType::GetType(ObjectType) == ECbFieldType::UniformObject;
		const ECbFieldType ExternalType = bUniformObject ? ValidateCbFieldType(ObjectView, Mode, Error) : ECbFieldType::HasFieldType;
		FCbUniformFieldsValidator UniformValidator(ExternalType);
		do
		{
			if (FCbFieldView Field = UniformValidator.ValidateField(ObjectView, Mode, Error))
			{
				if (EnumHasAnyFlags(Mode, ECbValidateMode::Names))
				{
					if (Field.HasName())
					{
						Names.Add(Field.GetName());
					}
					else
					{
						AddError(Error, ECbValidateError::MissingName);
					}
				}
			}
		}
		while (!ObjectView.IsEmpty());

		if (EnumHasAnyFlags(Mode, ECbValidateMode::Names) && Names.Num() > 1)
		{
			Algo::Sort(Names, [](FUtf8StringView L, FUtf8StringView R) { return L.Compare(R) < 0; });
			for (const FUtf8StringView* NamesIt = Names.GetData(), *NamesEnd = NamesIt + Names.Num() - 1; NamesIt != NamesEnd; ++NamesIt)
			{
				if (NamesIt[0].Equals(NamesIt[1]))
				{
					AddError(Error, ECbValidateError::DuplicateName);
					break;
				}
			}
		}

		if (!bUniformObject && EnumHasAnyFlags(Mode, ECbValidateMode::Format) && UniformValidator.IsUniform())
		{
			AddError(Error, ECbValidateError::NonUniformObject);
		}
	}
}

static void ValidateCbArray(FMemoryView& View, ECbValidateMode Mode, ECbValidateError& Error, ECbFieldType ArrayType)
{
	const uint64 Size = ValidateCbUInt(View, Mode, Error);
	FMemoryView ArrayView = View.Left(Size);
	View += Size;

	const uint64 Count = ValidateCbUInt(ArrayView, Mode, Error);
	const uint64 FieldsSize = ArrayView.GetSize();
	const bool bUniformArray = FCbFieldType::GetType(ArrayType) == ECbFieldType::UniformArray;
	const ECbFieldType ExternalType = bUniformArray ? ValidateCbFieldType(ArrayView, Mode, Error) : ECbFieldType::HasFieldType;
	FCbUniformFieldsValidator UniformValidator(ExternalType);

	for (uint64 Index = 0; Index < Count; ++Index)
	{
		if (FCbFieldView Field = UniformValidator.ValidateField(ArrayView, Mode, Error))
		{
			if (Field.HasName() && EnumHasAnyFlags(Mode, ECbValidateMode::Names))
			{
				AddError(Error, ECbValidateError::ArrayName);
			}
		}
	}

	if (!bUniformArray && EnumHasAnyFlags(Mode, ECbValidateMode::Format) && UniformValidator.IsUniform() && FieldsSize > Count)
	{
		AddError(Error, ECbValidateError::NonUniformArray);
	}
}

static FCbFieldView ValidateCbField(FMemoryView& View, ECbValidateMode Mode, ECbValidateError& Error, const ECbFieldType ExternalType = ECbFieldType::HasFieldType)
{
	const FMemoryView FieldView = View;
	const ECbFieldType Type = ValidateCbFieldType(View, Mode, Error, ExternalType);
	const FUtf8StringView Name = FCbFieldType::HasFieldName(Type) ? ValidateCbString(View, Mode, Error) : FUtf8StringView();

	if (EnumHasAnyFlags(Error, ECbValidateError::OutOfBounds | ECbValidateError::InvalidType))
	{
		return FCbFieldView();
	}

	switch (FCbFieldType::GetType(Type))
	{
	default:
	case ECbFieldType::None:
		AddError(Error, ECbValidateError::InvalidType);
		View.Reset();
		break;
	case ECbFieldType::Null:
	case ECbFieldType::BoolFalse:
	case ECbFieldType::BoolTrue:
		if (FieldView == View)
		{
			// Reset the view because a zero-sized field can cause infinite field iteration.
			AddError(Error, ECbValidateError::InvalidType);
			View.Reset();
		}
		break;
	case ECbFieldType::Object:
	case ECbFieldType::UniformObject:
		ValidateCbObject(View, Mode, Error, FCbFieldType::GetType(Type));
		break;
	case ECbFieldType::Array:
	case ECbFieldType::UniformArray:
		ValidateCbArray(View, Mode, Error, FCbFieldType::GetType(Type));
		break;
	case ECbFieldType::Binary:
		ValidateCbDynamicValue(View, Mode, Error);
		break;
	case ECbFieldType::String:
		ValidateCbString(View, Mode, Error);
		break;
	case ECbFieldType::IntegerPositive:
		ValidateCbUInt(View, Mode, Error);
		break;
	case ECbFieldType::IntegerNegative:
		ValidateCbUInt(View, Mode, Error);
		break;
	case ECbFieldType::Float32:
		ValidateCbFixedValue(View, Mode, Error, 4);
		break;
	case ECbFieldType::Float64:
		ValidateCbFloat64(View, Mode, Error);
		break;
	case ECbFieldType::ObjectAttachment:
	case ECbFieldType::BinaryAttachment:
	case ECbFieldType::Hash:
		ValidateCbFixedValue(View, Mode, Error, 20);
		break;
	case ECbFieldType::Uuid:
		ValidateCbFixedValue(View, Mode, Error, 16);
		break;
	case ECbFieldType::DateTime:
	case ECbFieldType::TimeSpan:
		ValidateCbFixedValue(View, Mode, Error, 8);
		break;
	case ECbFieldType::ObjectId:
		ValidateCbFixedValue(View, Mode, Error, 12);
		break;
	case ECbFieldType::CustomById:
	{
		FMemoryView Value = ValidateCbDynamicValue(View, Mode, Error);
		ValidateCbUInt(Value, Mode, Error);
		break;
	}
	case ECbFieldType::CustomByName:
	{
		FMemoryView Value = ValidateCbDynamicValue(View, Mode, Error);
		const FUtf8StringView TypeName = ValidateCbString(Value, Mode, Error);
		if (TypeName.IsEmpty() && !EnumHasAnyFlags(Error, ECbValidateError::OutOfBounds))
		{
			AddError(Error, ECbValidateError::InvalidType);
		}
		break;
	}
	}

	if (EnumHasAnyFlags(Error, ECbValidateError::OutOfBounds | ECbValidateError::InvalidType))
	{
		return FCbFieldView();
	}

	return FCbFieldView(FieldView.GetData(), ExternalType);
}

static FCbFieldView ValidateCbPackageField(FMemoryView& View, ECbValidateMode Mode, ECbValidateError& Error)
{
	if (View.IsEmpty())
	{
		if (EnumHasAnyFlags(Mode, ECbValidateMode::Package))
		{
			AddError(Error, ECbValidateError::InvalidPackageFormat);
		}
		return FCbFieldView();
	}
	if (FCbFieldView Field = ValidateCbField(View, Mode, Error))
	{
		if (Field.HasName() && EnumHasAnyFlags(Mode, ECbValidateMode::Package))
		{
			AddError(Error, ECbValidateError::InvalidPackageFormat);
		}
		return Field;
	}
	return FCbFieldView();
}

static FIoHash ValidateCbPackageAttachment(FCbFieldView& Value, FMemoryView& View, ECbValidateMode Mode, ECbValidateError& Error)
{
	if (const FCbObjectView ObjectView = Value.AsObjectView(); !Value.HasError())
	{
		return FCbObject().GetHash();
	}
	else if (const FIoHash ObjectAttachmentHash = Value.AsObjectAttachment(); !Value.HasError())
	{
		if (FCbFieldView ObjectField = ValidateCbPackageField(View, Mode, Error))
		{
			const FCbObjectView InnerObjectView = ObjectField.AsObjectView();
			if (EnumHasAnyFlags(Mode, ECbValidateMode::Package) && ObjectField.HasError())
			{
				AddError(Error, ECbValidateError::InvalidPackageFormat);
			}
			else if (EnumHasAnyFlags(Mode, ECbValidateMode::PackageHash) && (ObjectAttachmentHash != InnerObjectView.GetHash()))
			{
				AddError(Error, ECbValidateError::InvalidPackageHash);
			}
			return ObjectAttachmentHash;
		}
	}
	else if (const FIoHash BinaryAttachmentHash = Value.AsBinaryAttachment(); !Value.HasError())
	{
		if (FCbFieldView BinaryField = ValidateCbPackageField(View, Mode, Error))
		{
			const FMemoryView BinaryView = BinaryField.AsBinaryView();
			if (EnumHasAnyFlags(Mode, ECbValidateMode::Package) && BinaryField.HasError())
			{
				AddError(Error, ECbValidateError::InvalidPackageFormat);
			}
			else
			{
				if (EnumHasAnyFlags(Mode, ECbValidateMode::Package) && BinaryView.IsEmpty())
				{
					AddError(Error, ECbValidateError::NullPackageAttachment);
				}
				if (EnumHasAnyFlags(Mode, ECbValidateMode::PackageHash) && (BinaryAttachmentHash != FIoHash::HashBuffer(BinaryView)))
				{
					AddError(Error, ECbValidateError::InvalidPackageHash);
				}
			}
			return BinaryAttachmentHash;
		}
	}
	else if (const FMemoryView BinaryView = Value.AsBinaryView(); !Value.HasError())
	{
		if (BinaryView.GetSize() > 0)
		{
			FCompressedBuffer Buffer = FCompressedBuffer::FromCompressed(FSharedBuffer::MakeView(BinaryView));
			if (EnumHasAnyFlags(Mode, ECbValidateMode::Package) && Buffer.IsNull())
			{
				AddError(Error, ECbValidateError::NullPackageAttachment);
			}
			if (EnumHasAnyFlags(Mode, ECbValidateMode::PackageHash) && (Buffer.GetRawHash() != FIoHash::HashBuffer(Buffer.DecompressToComposite())))
			{
				AddError(Error, ECbValidateError::InvalidPackageHash);
			}
			return Buffer.GetRawHash();
		}
		else
		{
			if (EnumHasAnyFlags(Mode, ECbValidateMode::Package))
			{
				AddError(Error, ECbValidateError::NullPackageAttachment);
			}
			return FIoHash::HashBuffer(FMemoryView());
		}
	}
	else
	{
		if (EnumHasAnyFlags(Mode, ECbValidateMode::Package))
		{
			AddError(Error, ECbValidateError::InvalidPackageFormat);
		}
	}

	return FIoHash();
}

static FIoHash ValidateCbPackageObject(FCbFieldView& Value, FMemoryView& View, ECbValidateMode Mode, ECbValidateError& Error)
{
	if (FIoHash RootObjectHash = Value.AsHash(); !Value.HasError() && !Value.IsAttachment())
	{
		FCbFieldView RootObjectField = ValidateCbPackageField(View, Mode, Error);

		if (EnumHasAnyFlags(Mode, ECbValidateMode::Package))
		{
			if (RootObjectField.HasError())
			{
				AddError(Error, ECbValidateError::InvalidPackageFormat);
			}
		}

		const FCbObjectView RootObjectView = RootObjectField.AsObjectView();
		if (EnumHasAnyFlags(Mode, ECbValidateMode::Package))
		{
			if (!RootObjectView)
			{
				AddError(Error, ECbValidateError::NullPackageObject);
			}
		}

		if (EnumHasAnyFlags(Mode, ECbValidateMode::PackageHash) && (RootObjectHash != RootObjectView.GetHash()))
		{
			AddError(Error, ECbValidateError::InvalidPackageHash);
		}

		return RootObjectHash;
	}
	else
	{
		if (EnumHasAnyFlags(Mode, ECbValidateMode::Package))
		{
			AddError(Error, ECbValidateError::InvalidPackageFormat);
		}
	}

	return FIoHash();
}

static ECbValidateError ValidateCbFieldView(FCbFieldView Value, ECbValidateMode Mode)
{
	struct FCopy : public FCbFieldView
	{
		using FCbFieldView::GetType;
		using FCbFieldView::GetViewNoType;
	};
	const FCopy& Copy = static_cast<const FCopy&>(Value);
	return ValidateCompactBinary(Copy.GetViewNoType(), Mode, Copy.GetType());
}

ECbValidateError ValidateCompactBinary(FMemoryView View, ECbValidateMode Mode, ECbFieldType Type)
{
	ECbValidateError Error = ECbValidateError::None;
	if (EnumHasAnyFlags(Mode, ECbValidateMode::All))
	{
		ValidateCbField(View, Mode, Error, Type);
		if (!View.IsEmpty() && EnumHasAnyFlags(Mode, ECbValidateMode::Padding))
		{
			AddError(Error, ECbValidateError::Padding);
		}
	}
	return Error;
}

ECbValidateError ValidateCompactBinaryRange(FMemoryView View, ECbValidateMode Mode)
{
	ECbValidateError Error = ECbValidateError::None;
	if (EnumHasAnyFlags(Mode, ECbValidateMode::All))
	{
		while (!View.IsEmpty())
		{
			ValidateCbField(View, Mode, Error);
		}
	}
	return Error;
}

ECbValidateError ValidateCompactBinaryAttachment(FMemoryView View, ECbValidateMode Mode)
{
	ECbValidateError Error = ECbValidateError::None;
	if (EnumHasAnyFlags(Mode, ECbValidateMode::All))
	{
		if (FCbFieldView Value = ValidateCbPackageField(View, Mode, Error))
		{
			ValidateCbPackageAttachment(Value, View, Mode, Error);
		}
		if (!View.IsEmpty() && EnumHasAnyFlags(Mode, ECbValidateMode::Padding))
		{
			AddError(Error, ECbValidateError::Padding);
		}
	}
	return Error;
}

ECbValidateError ValidateCompactBinaryPackage(FMemoryView View, ECbValidateMode Mode)
{
	TArray<FIoHash, TInlineAllocator<16>> Attachments;
	ECbValidateError Error = ECbValidateError::None;
	if (EnumHasAnyFlags(Mode, ECbValidateMode::All))
	{
		uint32 ObjectCount = 0;
		while (FCbFieldView Value = ValidateCbPackageField(View, Mode, Error))
		{
			if (Value.IsHash() && !Value.IsAttachment())
			{
				ValidateCbPackageObject(Value, View, Mode, Error);
				if (++ObjectCount > 1 && EnumHasAnyFlags(Mode, ECbValidateMode::Package))
				{
					AddError(Error, ECbValidateError::MultiplePackageObjects);
				}
			}
			else if (Value.IsBinary() || Value.IsAttachment() || Value.IsObject())
			{
				const FIoHash Hash = ValidateCbPackageAttachment(Value, View, Mode, Error);
				if (EnumHasAnyFlags(Mode, ECbValidateMode::Package))
				{
					Attachments.Add(Hash);
				}
			}
			else if (Value.IsNull())
			{
				break;
			}
			else if (EnumHasAnyFlags(Mode, ECbValidateMode::Package))
			{
				AddError(Error, ECbValidateError::InvalidPackageFormat);
			}

			if (EnumHasAnyFlags(Error, ECbValidateError::OutOfBounds))
			{
				break;
			}
		}

		if (!View.IsEmpty() && EnumHasAnyFlags(Mode, ECbValidateMode::Padding))
		{
			AddError(Error, ECbValidateError::Padding);
		}

		if (Attachments.Num() && EnumHasAnyFlags(Mode, ECbValidateMode::Package))
		{
			Algo::Sort(Attachments);
			for (const FIoHash* It = Attachments.GetData(), *End = It + Attachments.Num() - 1; It != End; ++It)
			{
				if (It[0] == It[1])
				{
					AddError(Error, ECbValidateError::DuplicateAttachments);
					break;
				}
			}
		}
	}
	return Error;
}

ECbValidateError ValidateCompactBinary(const FCbField& Value, ECbValidateMode Mode)
{
	return ValidateCbFieldView(Value, Mode);
}

ECbValidateError ValidateCompactBinary(const FCbArray& Value, ECbValidateMode Mode)
{
	return ValidateCbFieldView(Value.AsFieldView(), Mode);
}

ECbValidateError ValidateCompactBinary(const FCbObject& Value, ECbValidateMode Mode)
{
	return ValidateCbFieldView(Value.AsFieldView(), Mode);
}

ECbValidateError ValidateCompactBinary(const FCbPackage& Value, ECbValidateMode Mode)
{
	ECbValidateError Error = ValidateCompactBinary(Value.GetObject(), Mode);

	for (const FCbAttachment& Attachment : Value.GetAttachments())
	{
		Error |= ValidateCompactBinary(Attachment, Mode);
	}

	if (EnumHasAnyFlags(Mode, ECbValidateMode::PackageHash) && (Value.GetObjectHash() != Value.GetObject().GetHash()))
	{
		AddError(Error, ECbValidateError::InvalidPackageHash);
	}

	return Error;
}

ECbValidateError ValidateCompactBinary(const FCbAttachment& Value, ECbValidateMode Mode)
{
	ECbValidateError Error = ECbValidateError::None;

	if (Value.IsObject())
	{
		Error |= ValidateCompactBinary(Value.AsObject(), Mode);
	}

	if (EnumHasAnyFlags(Mode, ECbValidateMode::PackageHash))
	{
		if (Value.IsObject())
		{
			if (Value.GetHash() != Value.AsObject().GetHash())
			{
				AddError(Error, ECbValidateError::InvalidPackageHash);
			}
		}
		else if (Value.IsBinary())
		{
			if (Value.GetHash() != FIoHash::HashBuffer(Value.AsCompositeBinary()))
			{
				AddError(Error, ECbValidateError::InvalidPackageHash);
			}
		}
		else if (Value.IsCompressedBinary())
		{
			const FCompressedBuffer& Buffer = Value.AsCompressedBinary();
			if (Buffer.GetRawHash() != FIoHash::HashBuffer(Buffer.DecompressToComposite()))
			{
				AddError(Error, ECbValidateError::InvalidPackageHash);
			}
		}
	}

	return Error;
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
