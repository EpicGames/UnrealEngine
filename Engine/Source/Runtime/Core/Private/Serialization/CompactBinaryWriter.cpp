// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/CompactBinaryWriter.h"

#include "Containers/AnsiString.h"
#include "Containers/Utf8String.h"
#include "HAL/PlatformString.h"
#include "HAL/UnrealMemory.h"
#include "IO/IoHash.h"
#include "Math/NumericLimits.h"
#include "Math/UnrealMathUtility.h"
#include "Memory/CompositeBuffer.h"
#include "Memory/SharedBuffer.h"
#include "Misc/AssertionMacros.h"
#include "Misc/ByteSwap.h"
#include "Misc/DateTime.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/Guid.h"
#include "Misc/Timespan.h"
#include "Serialization/Archive.h"
#include "Serialization/CompactBinaryPackage.h"
#include "Serialization/CompactBinarySerialization.h"
#include "Serialization/VarInt.h"
#include "UObject/NameTypes.h"

enum class FCbWriter::EStateFlags : uint8
{
	None = 0,
	/** Whether a name has been written for the current field. */
	Name = 1 << 0,
	/** Whether this state is in the process of writing a field. */
	Field = 1 << 1,
	/** Whether this state is for array fields. */
	Array = 1 << 2,
	/** Whether this state is for object fields. */
	Object = 1 << 3,
};

ENUM_CLASS_FLAGS(FCbWriter::EStateFlags);

/** Whether the field type can be used in a uniform array or uniform object. */
static constexpr bool IsUniformType(const ECbFieldType Type)
{
	if (FCbFieldType::HasFieldName(Type))
	{
		return true;
	}

	switch (Type)
	{
	case ECbFieldType::None:
	case ECbFieldType::Null:
	case ECbFieldType::BoolFalse:
	case ECbFieldType::BoolTrue:
		return false;
	default:
		return true;
	}
}

/** Append the value from the compact binary value to the array and return its type. */
static inline ECbFieldType AppendCompactBinary(const FCbFieldView& Value, TArray64<uint8>& OutData)
{
	struct FCopy : public FCbFieldView
	{
		using FCbFieldView::GetType;
		using FCbFieldView::GetValueView;
	};
	const FCopy& ValueCopy = static_cast<const FCopy&>(Value);
	const FMemoryView SourceView = ValueCopy.GetValueView();
	const int64 TargetOffset = OutData.AddUninitialized(SourceView.GetSize());
	MakeMemoryView(OutData).RightChop(TargetOffset).CopyFrom(SourceView);
	return ValueCopy.GetType();
}

FCbWriter::FCbWriter()
{
	States.Emplace();
}

FCbWriter::FCbWriter(const int64 InitialSize)
	: FCbWriter()
{
	Data.Reserve(InitialSize);
}

FCbWriter::~FCbWriter()
{
}

void FCbWriter::Reset()
{
	Data.Reset();
	States.Reset();
	States.Emplace();
}

FCbFieldIterator FCbWriter::Save() const
{
	AutoRTFM::UnreachableIfClosed("FUniqueBuffer is not yet transactionally safe. Call FCbWriter::Save(const FMutableMemoryView Buffer) instead.");

	const uint64 Size = GetSaveSize();
	FUniqueBuffer Buffer = FUniqueBuffer::Alloc(Size);
	const FCbFieldViewIterator Output = Save(Buffer);
	return FCbFieldIterator::MakeRangeView(Output, Buffer.MoveToShared());
}

FCbFieldViewIterator FCbWriter::Save(const FMutableMemoryView Buffer) const
{
	checkf(States.Num() == 1 && States.Last().Flags == EStateFlags::None,
		TEXT("It is invalid to save while there are incomplete write operations."));
	checkf(Data.Num() > 0, TEXT("It is invalid to save when nothing has been written."));
	checkf(Buffer.GetSize() == Data.Num(),
		TEXT("Buffer is %" UINT64_FMT " bytes but %" INT64_FMT " is required."), Buffer.GetSize(), Data.Num());
	Buffer.CopyFrom(MakeMemoryView(Data));
	return FCbFieldViewIterator::MakeRange(Buffer);
}

void FCbWriter::Save(FArchive& Ar) const
{
	check(Ar.IsSaving());
	checkf(States.Num() == 1 && States.Last().Flags == EStateFlags::None,
		TEXT("It is invalid to save while there are incomplete write operations."));
	checkf(Data.Num() > 0, TEXT("It is invalid to save when nothing has been written."));
	Ar.Serialize(const_cast<uint8*>(Data.GetData()), Data.Num());
}

uint64 FCbWriter::GetSaveSize() const
{
	return Data.Num();
}

void FCbWriter::BeginField()
{
	FState& State = States.Last();
	if ((State.Flags & EStateFlags::Field) == EStateFlags::None)
	{
		State.Flags |= EStateFlags::Field;
		State.Offset = Data.AddZeroed(sizeof(ECbFieldType));
	}
	else
	{
		checkf((State.Flags & EStateFlags::Name) == EStateFlags::Name, 
			TEXT("A new field cannot be written until the previous field '%s' is finished."), *WriteToString<64>(GetActiveName()));
	}
}

void FCbWriter::EndField(ECbFieldType Type)
{
	FState& State = States.Last();

	if ((State.Flags & EStateFlags::Name) == EStateFlags::Name)
	{
		Type |= ECbFieldType::HasFieldName;
	}
	else
	{
		checkf((State.Flags & EStateFlags::Object) == EStateFlags::None,
			TEXT("It is invalid to write an object field without a unique non-empty name."));
	}

	if (State.Count == 0)
	{
		State.UniformType = Type;
	}
	else if (State.UniformType != Type)
	{
		State.UniformType = ECbFieldType::None;
	}

	State.Flags &= ~(EStateFlags::Name | EStateFlags::Field);
	++State.Count;
	Data[State.Offset] = uint8(Type);
}

FCbWriter& FCbWriter::SetName(const FUtf8StringView Name)
{
	FState& State = States.Last();
	checkf((State.Flags & EStateFlags::Array) != EStateFlags::Array,
		TEXT("It is invalid to write a name for an array field. Name '%s'"), *WriteToString<64>(Name));
	checkf(!Name.IsEmpty(), TEXT("%s"), (State.Flags & EStateFlags::Object) == EStateFlags::Object
		? TEXT("It is invalid to write an empty name for an object field. Specify a unique non-empty name.")
		: TEXT("It is invalid to write an empty name for a top-level field. Specify a name or avoid this call."));
	checkf((State.Flags & (EStateFlags::Name | EStateFlags::Field)) == EStateFlags::None,
		TEXT("A new field '%s' cannot be written until the previous field '%s' is finished."), *WriteToString<64>(Name), *WriteToString<64>(GetActiveName()));

	BeginField();
	State.Flags |= EStateFlags::Name;
	const uint32 NameLenByteCount = MeasureVarUInt(uint32(Name.Len()));
	const int64 NameLenOffset = Data.AddUninitialized(NameLenByteCount);
	WriteVarUInt(uint64(Name.Len()), Data.GetData() + NameLenOffset);
	Data.Append(reinterpret_cast<const uint8*>(Name.GetData()), Name.Len());
	return *this;
}

void FCbWriter::SetNameOrAddString(const FUtf8StringView NameOrValue)
{
	// A name is only written if it would begin a new field inside of an object.
	if ((States.Last().Flags & (EStateFlags::Name | EStateFlags::Field | EStateFlags::Object)) == EStateFlags::Object)
	{
		SetName(NameOrValue);
	}
	else
	{
		AddString(NameOrValue);
	}
}

FUtf8StringView FCbWriter::GetActiveName() const
{
	const FState& State = States.Last();
	if ((State.Flags & EStateFlags::Name) == EStateFlags::Name)
	{
		const uint8* const EncodedName = Data.GetData() + State.Offset + sizeof(ECbFieldType);
		uint32 NameLenByteCount;
		const uint64 NameLen = ReadVarUInt(EncodedName, NameLenByteCount);
		const int32 ClampedNameLen = int32(FMath::Clamp<uint64>(NameLen, 0, MAX_int32));
		return FUtf8StringView(reinterpret_cast<const UTF8CHAR*>(EncodedName + NameLenByteCount), ClampedNameLen);
	}
	return FUtf8StringView();
}

void FCbWriter::MakeFieldsUniform(const int64 FieldBeginOffset, const int64 FieldEndOffset)
{
	FMutableMemoryView SourceView(Data.GetData() + FieldBeginOffset, uint64(FieldEndOffset - FieldBeginOffset));
	FMutableMemoryView TargetView = SourceView + sizeof(ECbFieldType);
	while (!SourceView.IsEmpty())
	{
		const uint64 FieldSize = MeasureCompactBinary(SourceView) - sizeof(ECbFieldType);
		SourceView += sizeof(ECbFieldType);
		if (TargetView.GetData() != SourceView.GetData())
		{
			FMemory::Memmove(TargetView.GetData(), SourceView.GetData(), FieldSize);
		}
		SourceView += FieldSize;
		TargetView += FieldSize;
	}
	if (!TargetView.IsEmpty())
	{
		Data.RemoveAt(FieldEndOffset - TargetView.GetSize(), TargetView.GetSize(), EAllowShrinking::No);
	}
}

void FCbWriter::AddField(const FCbFieldView& Value)
{
	checkf(Value.HasValue(), TEXT("It is invalid to write a field with no value."));
	BeginField();
	EndField(AppendCompactBinary(Value, Data));
}

void FCbWriter::AddField(const FCbField& Value)
{
	AddField(FCbFieldView(Value));
}

void FCbWriter::BeginObject()
{
	BeginField();
	States.Push(FState());
	States.Last().Flags |= EStateFlags::Object;
}

void FCbWriter::EndObject()
{
	checkf(States.Num() > 1 && (States.Last().Flags & EStateFlags::Object) == EStateFlags::Object,
		TEXT("It is invalid to end an object when an object is not at the top of the stack."));
	checkf((States.Last().Flags & EStateFlags::Field) == EStateFlags::None,
		TEXT("It is invalid to end an object until the previous field is finished."));
	const bool bUniform = IsUniformType(States.Last().UniformType);
	const uint64 Count = States.Last().Count;
	States.Pop();

	// Calculate the offset of the value.
	const FState& State = States.Last();
	int64 ValueOffset = State.Offset + 1;
	if ((State.Flags & EStateFlags::Name) == EStateFlags::Name)
	{
		uint32 NameLenByteCount;
		const uint64 NameLen = ReadVarUInt(Data.GetData() + ValueOffset, NameLenByteCount);
		ValueOffset += NameLen + NameLenByteCount;
	}

	// Remove redundant field types for uniform objects.
	if (bUniform && Count > 1)
	{
		MakeFieldsUniform(ValueOffset, Data.Num());
	}

	// Insert the object size.
	const uint64 Size = uint64(Data.Num() - ValueOffset);
	const uint32 SizeByteCount = MeasureVarUInt(Size);
	Data.InsertUninitialized(ValueOffset, SizeByteCount);
	WriteVarUInt(Size, Data.GetData() + ValueOffset);

	EndField(bUniform ? ECbFieldType::UniformObject : ECbFieldType::Object);
}

void FCbWriter::AddObject(const FCbObjectView& Value)
{
	BeginField();
	EndField(AppendCompactBinary(Value.AsFieldView(), Data));
}

void FCbWriter::AddObject(const FCbObject& Value)
{
	AddObject(FCbObjectView(Value));
}

void FCbWriter::BeginArray()
{
	BeginField();
	States.Push(FState());
	States.Last().Flags |= EStateFlags::Array;
}

void FCbWriter::EndArray()
{
	checkf(States.Num() > 1 && (States.Last().Flags & EStateFlags::Array) == EStateFlags::Array,
		TEXT("Invalid attempt to end an array when an array is not at the top of the stack."));
	checkf((States.Last().Flags & EStateFlags::Field) == EStateFlags::None,
		TEXT("It is invalid to end an array until the previous field is finished."));
	const bool bUniform = IsUniformType(States.Last().UniformType);
	const uint64 Count = States.Last().Count;
	States.Pop();

	// Calculate the offset of the value.
	const FState& State = States.Last();
	int64 ValueOffset = State.Offset + 1;
	if ((State.Flags & EStateFlags::Name) == EStateFlags::Name)
	{
		uint32 NameLenByteCount;
		const uint64 NameLen = ReadVarUInt(Data.GetData() + ValueOffset, NameLenByteCount);
		ValueOffset += NameLen + NameLenByteCount;
	}

	// Remove redundant field types for uniform arrays.
	if (bUniform && Count > 1)
	{
		MakeFieldsUniform(ValueOffset, Data.Num());
	}

	// Insert the array size and field count.
	const uint32 CountByteCount = MeasureVarUInt(Count);
	const uint64 Size = uint64(Data.Num() - ValueOffset) + CountByteCount;
	const uint32 SizeByteCount = MeasureVarUInt(Size);
	Data.InsertUninitialized(ValueOffset, SizeByteCount + CountByteCount);
	WriteVarUInt(Size, Data.GetData() + ValueOffset);
	WriteVarUInt(Count, Data.GetData() + ValueOffset + SizeByteCount);

	EndField(bUniform ? ECbFieldType::UniformArray : ECbFieldType::Array);
}

void FCbWriter::AddArray(const FCbArrayView& Value)
{
	BeginField();
	EndField(AppendCompactBinary(Value.AsFieldView(), Data));
}

void FCbWriter::AddArray(const FCbArray& Value)
{
	AddArray(FCbArrayView(Value));
}

void FCbWriter::AddNull()
{
	BeginField();
	EndField(ECbFieldType::Null);
}

void FCbWriter::AddBinary(const void* const Value, const uint64 Size)
{
	BeginField();
	const uint32 SizeByteCount = MeasureVarUInt(Size);
	const int64 SizeOffset = Data.AddUninitialized(SizeByteCount);
	WriteVarUInt(Size, Data.GetData() + SizeOffset);
	Data.Append(static_cast<const uint8*>(Value), Size);
	EndField(ECbFieldType::Binary);
}

void FCbWriter::AddBinary(FUtf8StringView Name, const FSharedBuffer& Value)
{
	SetName(Name);
	AddBinary(Value);
}

void FCbWriter::AddBinary(const FSharedBuffer& Buffer)
{
	AddBinary(Buffer.GetData(), Buffer.GetSize());
}

void FCbWriter::AddBinary(FUtf8StringView Name, const FCompositeBuffer& Value)
{
	SetName(Name);
	AddBinary(Value);
}

void FCbWriter::AddBinary(const FCompositeBuffer& Buffer)
{
	AddBinary(Buffer.ToShared());
}

void FCbWriter::AddString(const FUtf8StringView Value)
{
	BeginField();
	const uint64 Size = uint64(Value.Len());
	const uint32 SizeByteCount = MeasureVarUInt(Size);
	const int64 Offset = Data.AddUninitialized(SizeByteCount + Size);
	uint8* StringData = Data.GetData() + Offset;
	WriteVarUInt(Size, StringData);
	StringData += SizeByteCount;
	if (Size > 0)
	{
		FMemory::Memcpy(StringData, Value.GetData(), Value.Len() * sizeof(UTF8CHAR));
	}
	EndField(ECbFieldType::String);
}

void FCbWriter::AddString(const FWideStringView Value)
{
	BeginField();
	const uint32 Size = uint32(FPlatformString::ConvertedLength<UTF8CHAR>(Value.GetData(), Value.Len()));
	const uint32 SizeByteCount = MeasureVarUInt(Size);
	const int64 Offset = Data.AddUninitialized(SizeByteCount + Size);
	uint8* StringData = Data.GetData() + Offset;
	WriteVarUInt(Size, StringData);
	StringData += SizeByteCount;
	if (Size > 0)
	{
		FPlatformString::Convert(reinterpret_cast<UTF8CHAR*>(StringData), Size, Value.GetData(), Value.Len());
	}
	EndField(ECbFieldType::String);
}

void FCbWriter::AddInteger(const int32 Value)
{
	if (Value >= 0)
	{
		return AddInteger(uint32(Value));
	}
	BeginField();
	const uint32 Magnitude = ~uint32(Value);
	const uint32 MagnitudeByteCount = MeasureVarUInt(Magnitude);
	const int64 Offset = Data.AddUninitialized(MagnitudeByteCount);
	WriteVarUInt(Magnitude, Data.GetData() + Offset);
	EndField(ECbFieldType::IntegerNegative);
}

void FCbWriter::AddInteger(const int64 Value)
{
	if (Value >= 0)
	{
		return AddInteger(uint64(Value));
	}
	BeginField();
	const uint64 Magnitude = ~uint64(Value);
	const uint32 MagnitudeByteCount = MeasureVarUInt(Magnitude);
	const int64 Offset = Data.AddUninitialized(MagnitudeByteCount);
	WriteVarUInt(Magnitude, Data.GetData() + Offset);
	EndField(ECbFieldType::IntegerNegative);
}

void FCbWriter::AddInteger(const uint32 Value)
{
	BeginField();
	const uint32 ValueByteCount = MeasureVarUInt(Value);
	const int64 Offset = Data.AddUninitialized(ValueByteCount);
	WriteVarUInt(Value, Data.GetData() + Offset);
	EndField(ECbFieldType::IntegerPositive);
}

void FCbWriter::AddInteger(const uint64 Value)
{
	BeginField();
	const uint32 ValueByteCount = MeasureVarUInt(Value);
	const int64 Offset = Data.AddUninitialized(ValueByteCount);
	WriteVarUInt(Value, Data.GetData() + Offset);
	EndField(ECbFieldType::IntegerPositive);
}

void FCbWriter::AddFloat(const float Value)
{
	BeginField();
	const uint32 RawValue = NETWORK_ORDER32(reinterpret_cast<const uint32&>(Value));
	Data.Append(reinterpret_cast<const uint8*>(&RawValue), sizeof(uint32));
	EndField(ECbFieldType::Float32);
}

void FCbWriter::AddFloat(const double Value)
{
	const float Value32 = float(Value);
	if (Value == double(Value32))
	{
		return AddFloat(Value32);
	}
	BeginField();
	const uint64 RawValue = NETWORK_ORDER64(reinterpret_cast<const uint64&>(Value));
	Data.Append(reinterpret_cast<const uint8*>(&RawValue), sizeof(uint64));
	EndField(ECbFieldType::Float64);
}

void FCbWriter::AddBool(const bool bValue)
{
	BeginField();
	EndField(bValue ? ECbFieldType::BoolTrue : ECbFieldType::BoolFalse);
}

void FCbWriter::AddObjectAttachment(const FIoHash& Value)
{
	BeginField();
	Data.Append(Value.GetBytes(), sizeof(decltype(Value.GetBytes())));
	EndField(ECbFieldType::ObjectAttachment);
}

void FCbWriter::AddBinaryAttachment(const FIoHash& Value)
{
	BeginField();
	Data.Append(Value.GetBytes(), sizeof(decltype(Value.GetBytes())));
	EndField(ECbFieldType::BinaryAttachment);
}

void FCbWriter::AddAttachment(const FCbAttachment& Attachment)
{
	BeginField();
	const FIoHash& Value = Attachment.GetHash();
	Data.Append(Value.GetBytes(), sizeof(decltype(Value.GetBytes())));
	EndField(ECbFieldType::BinaryAttachment);
}

void FCbWriter::AddHash(const FIoHash& Value)
{
	BeginField();
	Data.Append(Value.GetBytes(), sizeof(decltype(Value.GetBytes())));
	EndField(ECbFieldType::Hash);
}

void FCbWriter::AddUuid(const FGuid& Value)
{
	const auto AppendSwappedBytes = [this](uint32 In)
	{
		In = NETWORK_ORDER32(In);
		Data.Append(reinterpret_cast<const uint8*>(&In), sizeof(In));
	};
	BeginField();
	AppendSwappedBytes(Value.A);
	AppendSwappedBytes(Value.B);
	AppendSwappedBytes(Value.C);
	AppendSwappedBytes(Value.D);
	EndField(ECbFieldType::Uuid);
}

void FCbWriter::AddDateTimeTicks(const int64 Ticks)
{
	BeginField();
	const uint64 RawValue = NETWORK_ORDER64(uint64(Ticks));
	Data.Append(reinterpret_cast<const uint8*>(&RawValue), sizeof(uint64));
	EndField(ECbFieldType::DateTime);
}

void FCbWriter::AddDateTime(FUtf8StringView Name, FDateTime Value)
{
	SetName(Name);
	AddDateTime(Value);
}

void FCbWriter::AddDateTime(const FDateTime Value)
{
	AddDateTimeTicks(Value.GetTicks());
}

void FCbWriter::AddTimeSpanTicks(const int64 Ticks)
{
	BeginField();
	const uint64 RawValue = NETWORK_ORDER64(uint64(Ticks));
	Data.Append(reinterpret_cast<const uint8*>(&RawValue), sizeof(uint64));
	EndField(ECbFieldType::TimeSpan);
}

void FCbWriter::AddTimeSpan(FUtf8StringView Name, FTimespan Value)
{
	SetName(Name);
	AddTimeSpan(Value);
}

void FCbWriter::AddTimeSpan(const FTimespan Value)
{
	AddTimeSpanTicks(Value.GetTicks());
}

void FCbWriter::AddObjectId(const FCbObjectId& Value)
{
	static_assert(sizeof(FCbObjectId) == 12, "FCbObjectId is expected to be 12 bytes.");
	BeginField();
	Data.Append(reinterpret_cast<const uint8*>(&Value), sizeof(FCbObjectId));
	EndField(ECbFieldType::ObjectId);
}

void FCbWriter::AddCustom(const uint64 TypeId, const FMemoryView Value)
{
	BeginField();
	const uint32 TypeByteCount = MeasureVarUInt(TypeId);
	const uint64 Size = TypeByteCount + Value.GetSize();
	const uint32 SizeByteCount = MeasureVarUInt(Size);
	const int64 ValueOffset = Data.AddUninitialized(SizeByteCount + TypeByteCount);
	WriteVarUInt(Size, Data.GetData() + ValueOffset);
	WriteVarUInt(TypeId, Data.GetData() + ValueOffset + SizeByteCount);
	Data.Append(static_cast<const uint8*>(Value.GetData()), Value.GetSize());
	EndField(ECbFieldType::CustomById);
}

void FCbWriter::AddCustom(const FUtf8StringView TypeName, const FMemoryView Value)
{
	checkf(!TypeName.IsEmpty(), TEXT("Field '%s' requires a non-empty type name for its custom type."), *WriteToString<64>(GetActiveName()));

	BeginField();

	const uint64 TypeNameLen = TypeName.Len();
	const uint32 TypeNameLenByteCount = MeasureVarUInt(TypeNameLen);

	const uint64 Size = TypeNameLenByteCount + TypeNameLen + Value.GetSize();
	const uint32 SizeByteCount = MeasureVarUInt(Size);

	const int64 ValueOffset = Data.AddUninitialized(SizeByteCount + TypeNameLenByteCount);
	WriteVarUInt(Size, Data.GetData() + ValueOffset);
	WriteVarUInt(TypeNameLen, Data.GetData() + ValueOffset + SizeByteCount);
	Data.Append(reinterpret_cast<const uint8*>(TypeName.GetData()), TypeNameLen);
	Data.Append(static_cast<const uint8*>(Value.GetData()), Value.GetSize());

	EndField(ECbFieldType::CustomByName);
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FCbWriter& FCbWriter::operator<<(const FDateTime Value)
{
	AddDateTime(Value);
	return *this;
}

FCbWriter& FCbWriter::operator<<(const FTimespan Value)
{
	AddTimeSpan(Value);
	return *this;
}

FCbWriter& FCbWriter::operator<<(const FString& Value)
{
	*this << FStringView(Value);
	return *this;
}

FCbWriter& FCbWriter::operator<<(const FAnsiString& Value)
{
	*this << FAnsiStringView(Value);
	return *this;
}

FCbWriter& FCbWriter::operator<<(const FUtf8String& Value)
{
	*this << FUtf8StringView(Value);
	return *this;
}

FCbWriter& FCbWriter::operator<<(FName Value)
{
	*this << WriteToUtf8String<FName::StringBufferSize>(Value).ToView();
	return *this;
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
