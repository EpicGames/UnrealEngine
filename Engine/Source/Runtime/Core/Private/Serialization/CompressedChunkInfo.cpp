// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/CompressedChunkInfo.h"
#include "Serialization/Archive.h"

/** 
 * FCompressedChunkInfo serialize operator.
 */
FArchive& operator<<( FArchive& Ar, FCompressedChunkInfo& Chunk )
{
	// The order of serialization needs to be identical to the memory layout as the async IO code is reading it in bulk.
	// The size of the structure also needs to match what is being serialized.
	Ar << Chunk.CompressedSize;
	Ar << Chunk.UncompressedSize;
	return Ar;
}
