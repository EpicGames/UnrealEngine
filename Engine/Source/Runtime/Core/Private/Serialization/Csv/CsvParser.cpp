// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/Csv/CsvParser.h"

#include "Templates/UnrealTemplate.h"

FCsvParser::FCsvParser(FString&& InSourceString)
	: SourceString(MoveTemp(InSourceString))
{
	if (!SourceString.IsEmpty())
	{
		ParseRows();
	}
}

FCsvParser::FCsvParser(const FString& InSourceString)
	: SourceString(InSourceString)
{
	if (!SourceString.IsEmpty())
	{
		ParseRows();
	}
}

void FCsvParser::ParseRows()
{
	BufferStart = &SourceString[0];
	ReadAt = BufferStart;

	EParseResult Result;
	do
	{
		Result = ParseRow();
	} while(Result != EParseResult::EndOfString);
}

FCsvParser::EParseResult FCsvParser::ParseRow()
{
	// Check for an empty line
	const int8 NewLineSize = MeasureNewLine(ReadAt);
	if (NewLineSize)
	{
		ReadAt += NewLineSize;
		return *ReadAt ? EParseResult::EndOfRow : EParseResult::EndOfString;
	}

	EParseResult Result;

	Rows.Emplace();
	do
	{
		Result = ParseCell();
	}
	while(Result == EParseResult::EndOfCell);

	return Result;
}

FCsvParser::EParseResult FCsvParser::ParseCell()
{
	TCHAR* WriteAt = const_cast<TCHAR*>(ReadAt);

	// Check if this cell is quoted. Whitespace between cell opening and quote is invalid.
	bool bQuoted = *ReadAt == '"';

	if (bQuoted)
	{
		// Skip over the first quote
		ReadAt = ++WriteAt;
	}
	
	Rows.Last().Add(ReadAt);

	while( *ReadAt )
	{
		// If the cell value is quoted, we swallow everything until we find a closing quote
		if (bQuoted)
		{
			if (*ReadAt == '"')
			{
				// RFC 4180 specifies that double quotes are escaped as ""

				while (*ReadAt == '"')
				{
					++ReadAt;
					bQuoted = !bQuoted;

					// Unescape the double quotes
					if (bQuoted)
					{
						*WriteAt++ = TEXT('"');
					}
				}

				// We null terminate and leave the write pos pointing at the trailing closing quote 
				// if present so it gets overwritten by any subsequent text in the cell
				*WriteAt = TEXT('\0');

				continue;
			}
		}
		else
		{
			// Check for the end of a row (new line)
			const int8 NewLineSize = MeasureNewLine(ReadAt);
			if (NewLineSize != 0)
			{
				// Null terminate the cell
				*WriteAt = TEXT('\0');
				ReadAt += NewLineSize;

				return *ReadAt ? EParseResult::EndOfRow : EParseResult::EndOfString;
			}
			else if (*ReadAt == ',')
			{
				*WriteAt = TEXT('\0');
				++ReadAt;

				// We always return EndOfCell here as we still have another (potentially empty) cell to add
				// In the case where ReadAt now points at the string terminator, the next call to ParseCell 
				// will add an empty cell and then return EndOfString
				return EParseResult::EndOfCell;
			}
		}
		if (WriteAt != ReadAt)
		{
			(*WriteAt++) = (*ReadAt++);
		}
		else
		{
			ReadAt = ++WriteAt;
		}
	}

	return EParseResult::EndOfString;
}

int8 FCsvParser::MeasureNewLine(const TCHAR* At)
{
	switch(*At)
	{
		case '\r':
			if (*(At+1) == '\n')	// could be null
			{
				return 2;
			}
			return 1;
		case '\n':
			return 1;
		default:
			return 0;
	}
}
