// Copyright Epic Games, Inc. All Rights Reserved.

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Misc/AutomationTest.h"
#include "Misc/CString.h"
#include "Serialization/Csv/CsvParser.h"
#include "Templates/UnrealTemplate.h"

namespace CsvParser_Tests
{

	/** Check that the specified row of cells matches what we expect */
	template <uint32 N>
	bool CheckRow(const TArray<const TCHAR*>& Row, const FString (&Expected)[N], int32 RowIndex, FAutomationTestBase* Test)
	{
		if (Row.Num() != N)
		{
			Test->AddError(FString::Printf(TEXT("Row %d has the wrong number of cells. Expected %d, but parsed result has %d."), RowIndex, N, Row.Num()));
			return false;
		}

		for (int32 Cell = 0; Cell < Row.Num(); ++Cell)
		{
			if (FCString::Strcmp(Row[Cell], *Expected[Cell]) != 0)
			{
				Test->AddError(FString::Printf(TEXT("Cell %d on row %d did not parse correctly. Expected:\n%s\nBut parsed result was:\n%s"), Cell, RowIndex, *Expected[Cell], Row[Cell]));
				return false;
			}
		}

		return true;
	}
	/** Check that the specified rows all match the specified results */
	template <uint32 RowCount, uint32 CellCount>
	bool CheckRows(const FCsvParser::FRows& Rows, const FString (&Expected)[RowCount][CellCount], FAutomationTestBase* Test)
	{
		if (Rows.Num() != RowCount)
		{
			Test->AddError(FString::Printf(TEXT("Parser generated wrong number of rows. Expected %d, but parsed result has %d."), RowCount, Rows.Num()));
			return false;
		}

		for (int32 Row = 0; Row < RowCount; ++Row)
		{
			if (!CheckRow(Rows[Row], Expected[Row], Row, Test))
			{
				return false;
			}
		}

		return true;
	}
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FBasicTest, "System.Core.CSV Parser.Basic", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FBasicTest::RunTest(const FString& Parameters)
{
	FString Str =	"1,2,3,4\n5,6,7,8";
	const FCsvParser Parser(MoveTemp(Str));

	const FString Expected[][4] = {
		{ "1", "2", "3", "4" },
		{ "5", "6", "7", "8" }
	};

	return CsvParser_Tests::CheckRows(Parser.GetRows(), Expected, this);
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FQuotedStringTest, "System.Core.CSV Parser.Quoted String", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FQuotedStringTest::RunTest(const FString& Parameters)
{
	FString Str =	"\"Quoted with nested \"\"quotes\"\", and \"\"commas\"\"\" \"unquoted due to \"whitespace\" , second \" unquoted\n"
					"\"Quoted\nString,With\nNewlines\",\"\"\n"
					"1,\"2\"\n";
	const FCsvParser Parser(MoveTemp(Str));

	const FString Expected[][2] = {
		{ "Quoted with nested \"quotes\", and \"commas\" \"unquoted due to \"whitespace\" ", " second \" unquoted" },
		{ "Quoted\nString,With\nNewlines", "" },
		{ "1", "2" },
	};

	return CsvParser_Tests::CheckRows(Parser.GetRows(), Expected, this);
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FEmptyCellsTest, "System.Core.CSV Parser.Empty Cells", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FEmptyCellsTest::RunTest(const FString& Parameters)
{
	FString Str =	",,,\n"
					",,,\n"
					" , , , \n";
	const FCsvParser Parser(MoveTemp(Str));

	const FString Expected[][4] = {
		{ "", "", "", "" },
		{ "", "", "", "" },
		{ " ", " ", " ", " " },	// Whitespace is significant in csv
	};

	return CsvParser_Tests::CheckRows(Parser.GetRows(), Expected, this);
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMixedNewLinesTest, "System.Core.CSV Parser.Mixed Newlines", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FMixedNewLinesTest::RunTest(const FString& Parameters)
{
	const FCsvParser Parser(FString("1\r2\n3\r\n4\n\n5\r\r6\r\n\r\n7\n\r8"));

	const FString Expected[][1] = {
		{ "1" }, { "2" }, { "3" }, { "4" }, { "5" }, { "6" }, { "7" }, { "8" },
	};

	return CsvParser_Tests::CheckRows(Parser.GetRows(), Expected, this);
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FEmptyStringTest, "System.Core.CSV Parser.Empty String", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FEmptyStringTest::RunTest(const FString& Parameters)
{
	return	FCsvParser(FString()).GetRows().Num() == 0 &&
			FCsvParser(FString("")).GetRows().Num() == 0;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FOnlyNewLinesTest, "System.Core.CSV Parser.Only Newlines", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FOnlyNewLinesTest::RunTest(const FString& Parameters)
{
	return	FCsvParser(FString("\r\n\r\r\n\n\r\n\n\r")).GetRows().Num() == 0;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FEndOfStringTest, "System.Core.CSV Parser.End of String", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FEndOfStringTest::RunTest(const FString& Parameters)
{
	bool bSuccess = true;

	{
		const FCsvParser Parser(TEXT(",,,"));

		const FString Expected[][4] = {
			{ "", "", "", "" },
		};

		bSuccess &= CsvParser_Tests::CheckRows(Parser.GetRows(), Expected, this);
	}

	{
		const FCsvParser Parser(TEXT("\"\",\"\",\"\",\"\""));

		const FString Expected[][4] = {
			{ "", "", "", "" },
		};

		bSuccess &= CsvParser_Tests::CheckRows(Parser.GetRows(), Expected, this);
	}

	{
		const FCsvParser Parser(TEXT(",,,\n"));

		const FString Expected[][4] = {
			{ "", "", "", "" },
		};

		bSuccess &= CsvParser_Tests::CheckRows(Parser.GetRows(), Expected, this);
	}

	{
		const FCsvParser Parser(TEXT("\"\",\"\",\"\",\"\"\n"));

		const FString Expected[][4] = {
			{ "", "", "", "" },
		};

		bSuccess &= CsvParser_Tests::CheckRows(Parser.GetRows(), Expected, this);
	}

	return bSuccess;
}
