// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	CustomVersion.cpp: Unreal custom versioning system.
=============================================================================*/

#include "Serialization/CustomVersion.h"

#include "Algo/Sort.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/Map.h"
#include "HAL/CriticalSection.h"
#include "Misc/AssertionMacros.h"
#include "Misc/ScopeRWLock.h"
#include "Misc/TransactionallySafeRWLock.h"
#include "Serialization/Archive.h"
#include "Serialization/StructuredArchive.h"
#include "Serialization/StructuredArchiveAdapters.h"
#include "Serialization/StructuredArchiveNameHelpers.h"
#include "Serialization/StructuredArchiveSlots.h"
#include "UObject/UnrealNames.h"
#include "AutoRTFM.h"

namespace
{
	static const FGuid UnusedCustomVersionKey(0, 0, 0, 0xF99D40C1);

	static const FCustomVersion& GetUnusedCustomVersion()
	{
		static const FCustomVersion UnusedCustomVersion(UnusedCustomVersionKey, 0, TEXT("Unused custom version"));
		return UnusedCustomVersion;
	}
	
	struct FEnumCustomVersion_DEPRECATED
	{
		uint32 Tag;
		int32  Version;

		FCustomVersion ToCustomVersion() const
		{
			// We'll invent a GUID from three zeroes and the original tag
			return FCustomVersion(FGuid(0, 0, 0, Tag), Version, *FString::Printf(TEXT("EnumTag%u"), Tag));
		}
	};

	void operator<<(FStructuredArchive::FSlot Slot, FEnumCustomVersion_DEPRECATED& Version)
	{
		// Serialize keys
		FStructuredArchive::FRecord Record = Slot.EnterRecord();
		Record << SA_VALUE(TEXT("Tag"), Version.Tag);
		Record << SA_VALUE(TEXT("Version"), Version.Version);
	}

	FArchive& operator<<(FArchive& Ar, FEnumCustomVersion_DEPRECATED& Version)
	{
		FStructuredArchiveFromArchive(Ar).GetSlot() << Version;
		return Ar;
	}

	struct FGuidCustomVersion_DEPRECATED
	{
		FGuid Key;
		int32 Version;
		FString FriendlyName;

		FCustomVersion ToCustomVersion() const
		{
			return FCustomVersion(Key, Version, *FriendlyName);
		}
	};

	void operator<<(FStructuredArchive::FSlot Slot, FGuidCustomVersion_DEPRECATED& Version)
	{
		FStructuredArchive::FRecord Record = Slot.EnterRecord();
		Record << SA_VALUE(TEXT("Key"), Version.Key);
		Record << SA_VALUE(TEXT("Version"), Version.Version);
		Record << SA_VALUE(TEXT("FriendlyName"), Version.FriendlyName);
	}

	FArchive& operator<<(FArchive& Ar, FGuidCustomVersion_DEPRECATED& Version)
	{
		FStructuredArchiveFromArchive(Ar).GetSlot() << Version;
		return Ar;
	}
}

/** Defer FName creation and allocations from static FCustomVersionRegistrations that may never be needed */
struct FStaticCustomVersionRegistry
{
	struct FPendingRegistration
	{
		int32 Version;
		const TCHAR* FriendlyName;
		CustomVersionValidatorFunc ValidatorFunc;
	};
	typedef TMap<FGuid, FPendingRegistration, TInlineSetAllocator<64>> RegistrationQueue;

	FTransactionallySafeRWLock Lock;
	FCustomVersionContainer Registered;
	RegistrationQueue Queue;

	static FStaticCustomVersionRegistry& Get()
	{
		static FStaticCustomVersionRegistry Singleton;
		return Singleton;
	}

	TOptional<FCustomVersion> Find(const FGuid& Guid) const
	{
		if (const FCustomVersion* RegisteredVersion = Registered.GetVersion(Guid))
		{
			return *RegisteredVersion;
		}

		if (const FPendingRegistration* Pending = Queue.Find(Guid))
		{
			return FCustomVersion(Guid, Pending->Version, Pending->FriendlyName, Pending->ValidatorFunc);
		}

		return TOptional<FCustomVersion>();
	}

	void RegisterQueue()
	{
		if (Queue.Num())
		{
			for (const TPair<FGuid, FPendingRegistration>& Queued : Queue)
			{
				// Check if this tag hasn't already been registered
				if (FCustomVersion* ExistingRegistration = Registered.Versions.FindByKey(Queued.Key))
				{
					// We don't allow the registration details to change across registrations - this code path only exists to support hotreload

					// If you hit this then you've probably either:
					// * Changed registration details during hotreload.
					// * Accidentally copy-and-pasted an FCustomVersionRegistration object.
					ensureMsgf(
						ExistingRegistration->Version == Queued.Value.Version && ExistingRegistration->GetFriendlyName() == Queued.Value.FriendlyName,
						TEXT("Custom version registrations cannot change between hotreloads - \"%s\" version %d is being reregistered as \"%s\" version %d"),
						*ExistingRegistration->GetFriendlyName().ToString(),
						ExistingRegistration->Version,
						Queued.Value.FriendlyName,
						Queued.Value.Version
					);

					++ExistingRegistration->ReferenceCount;
				}
				else
				{
					Registered.Versions.Add(FCustomVersion(Queued.Key, Queued.Value.Version, Queued.Value.FriendlyName, Queued.Value.ValidatorFunc));
				}
			}

			Queue.Empty();
		}
	}

	void Unregister(FGuid Key)
	{
		if (Queue.Remove(Key) == 0)
		{
			const int32 KeyIndex = Registered.Versions.IndexOfByKey(Key);

			// Ensure this tag has been registered
			check(KeyIndex != INDEX_NONE);

			FCustomVersion* FoundKey = &Registered.Versions[KeyIndex];

			--FoundKey->ReferenceCount;
			if (FoundKey->ReferenceCount == 0)
			{
				Registered.Versions.RemoveAtSwap(KeyIndex);
			}
		}
	}
};

FCustomVersionContainer FCurrentCustomVersions::GetAll()
{
	FStaticCustomVersionRegistry& Registry = FStaticCustomVersionRegistry::Get();

	{
		UE::TReadScopeLock Scope(Registry.Lock);

		if (Registry.Queue.Num() == 0)
		{
			return Registry.Registered;
		}
	}

	UE::TWriteScopeLock Scope(Registry.Lock);
	Registry.RegisterQueue();
	return Registry.Registered;
}

TOptional<FCustomVersion> FCurrentCustomVersions::Get(const FGuid& Guid)
{
	FStaticCustomVersionRegistry& Registry = FStaticCustomVersionRegistry::Get();

	UE::TReadScopeLock Scope(Registry.Lock);
	return Registry.Find(Guid);
}

TArray<FCustomVersionDifference> FCurrentCustomVersions::Compare(const FCustomVersionArray& CompareVersions, const TCHAR* DebugContext)
{
	TArray<FCustomVersionDifference> Result;

	if (CompareVersions.Num())
	{
		FStaticCustomVersionRegistry& Registry = FStaticCustomVersionRegistry::Get();

		UE::TReadScopeLock Scope(Registry.Lock);

		for (const FCustomVersion& CompareVersion : CompareVersions)
		{
			if (TOptional<FCustomVersion> CurrentVersion = Registry.Find(CompareVersion.Key))
			{
				if (CurrentVersion.GetValue().Validator && !CurrentVersion.GetValue().Validator(CompareVersion, CompareVersions, DebugContext))
				{
					Result.Add({ ECustomVersionDifference::Invalid, &CompareVersion });
				}
				else if (int Delta = CurrentVersion.GetValue().Version - CompareVersion.Version)
				{
					Result.Add({ Delta < 0	? ECustomVersionDifference::Newer 
											: ECustomVersionDifference::Older, &CompareVersion });
				}
			}
			else
			{
				Result.Add({ ECustomVersionDifference::Missing, &CompareVersion });
			}			
		}
	}

	return Result;
}

void FCurrentCustomVersions::Register(const FGuid& Key, int32 Version, const TCHAR* Name, CustomVersionValidatorFunc ValidatorFunc)
{
	FStaticCustomVersionRegistry& Registry = FStaticCustomVersionRegistry::Get();

	UE::TWriteScopeLock Scope(Registry.Lock);
	check(Registry.Queue.Find(Key) == nullptr);
	Registry.Queue.Add(Key, { Version, Name, ValidatorFunc });
}

void FCurrentCustomVersions::Unregister(const FGuid& Key)
{
	FStaticCustomVersionRegistry& Registry = FStaticCustomVersionRegistry::Get();

	UE::TWriteScopeLock Scope(Registry.Lock);
	Registry.Unregister(Key);
}

const FName FCustomVersion::GetFriendlyName() const
{
	if (FriendlyName == NAME_None)
	{
		if (TOptional<FCustomVersion> CurrentVersion = FCurrentCustomVersions::Get(Key))
		{
			FriendlyName = CurrentVersion.GetValue().GetFriendlyName();
		}
	}
	return FriendlyName;
}

void FCustomVersionContainer::Empty()
{
	Versions.Empty();
}

void FCustomVersionContainer::SortByKey()
{
	Algo::SortBy(Versions, &FCustomVersion::Key);
}

FString FCustomVersionContainer::ToString(const FString& Indent) const
{
	TStringBuilder<2048> VersionsAsString;
	for (const FCustomVersion& SomeVersion : Versions)
	{
		VersionsAsString << Indent
			<< TEXTVIEW("Key=") << SomeVersion.Key
			<< TEXTVIEW(" Version = ") << SomeVersion.Version
			<< TEXTVIEW(" Friendly Name = ") << SomeVersion.GetFriendlyName()
			<< TEXTVIEW("\n");
	}

	return FString(VersionsAsString);
}

FArchive& operator<<(FArchive& Ar, FCustomVersion& Version)
{
	FStructuredArchiveFromArchive(Ar).GetSlot() << Version;
	return Ar;
}

void operator<<(FStructuredArchive::FSlot Slot, FCustomVersion& Version)
{
	FStructuredArchive::FRecord Record = Slot.EnterRecord();
	Record << SA_VALUE(TEXT("Key"), Version.Key);
	Record << SA_VALUE(TEXT("Version"), Version.Version);
}

void FCustomVersionContainer::Serialize(FArchive& Ar, ECustomVersionSerializationFormat Format)
{
	Serialize(FStructuredArchiveFromArchive(Ar).GetSlot(), Format);
}

void FCustomVersionContainer::Serialize(FStructuredArchive::FSlot Slot, ECustomVersionSerializationFormat Format)
{
	switch (Format)
	{
	default: check(false);

	case ECustomVersionSerializationFormat::Enums:
	{
		// We should only ever be loading enums.  They should never be saved - they only exist for backward compatibility.
		check(Slot.GetUnderlyingArchive().IsLoading());

		TArray<FEnumCustomVersion_DEPRECATED> OldTags;
		Slot << OldTags;

		Versions.Empty(OldTags.Num());
		for (auto It = OldTags.CreateConstIterator(); It; ++It)
		{
			Versions.Add(It->ToCustomVersion());
		}
	}
	break;

	case ECustomVersionSerializationFormat::Guids:
	{
		// We should only ever be loading old versions.  They should never be saved - they only exist for backward compatibility.
		check(Slot.GetUnderlyingArchive().IsLoading());

		TArray<FGuidCustomVersion_DEPRECATED> VersionArray;
		Slot << VersionArray;
		Versions.Empty(VersionArray.Num());
		for (FGuidCustomVersion_DEPRECATED& OldVer : VersionArray)
		{
			Versions.Add(OldVer.ToCustomVersion());
		}
	}
	break;

	case ECustomVersionSerializationFormat::Optimized:
	{
		Slot << Versions;
	}
	break;
	}
}

const FCustomVersion* FCustomVersionContainer::GetVersion(FGuid Key) const
{
	// A testing tag was written out to a few archives during testing so we need to
	// handle the existence of it to ensure that those archives can still be loaded.
	if (Key == UnusedCustomVersionKey)
	{
		return &GetUnusedCustomVersion();
	}

	return Versions.FindByKey(Key);
}

const FName FCustomVersionContainer::GetFriendlyName(FGuid Key) const
{
	FName FriendlyName = NAME_None;
	const FCustomVersion* CustomVersion = GetVersion(Key);
	if (CustomVersion)
	{
		FriendlyName = CustomVersion->FriendlyName;
	}
	return FriendlyName;
}

void FCustomVersionContainer::SetVersion(FGuid CustomKey, int32 Version, FName FriendlyName)
{
	if (CustomKey == UnusedCustomVersionKey)
	{
		return;
	}

	if (FCustomVersion* Found = Versions.FindByKey(CustomKey))
	{
		Found->Version      = Version;
		Found->FriendlyName = FriendlyName;
	}
	else
	{
		Versions.Add(FCustomVersion(CustomKey, Version, FriendlyName));
	}
}

void FCustomVersionContainer::SetVersionUsingRegistry(FGuid CustomKey, ESetCustomVersionFlags Options)
{
	if (CustomKey == UnusedCustomVersionKey)
	{
		return;
	}

	if (FCustomVersion* Found = Versions.FindByKey(CustomKey))
	{
		if (EnumHasAnyFlags(Options, ESetCustomVersionFlags::SkipUpdateExistingVersion))
		{
			return;
		}

		TOptional<FCustomVersion> RegisteredVersion;
		UE_AUTORTFM_OPEN{
			RegisteredVersion = FCurrentCustomVersions::Get(CustomKey);
		};
		checkf(RegisteredVersion, TEXT("Attempted to set a version that is not registered"));

		Found->Version      = RegisteredVersion->Version;
		Found->FriendlyName = RegisteredVersion->FriendlyName;
	}
	else
	{
		TOptional<FCustomVersion> RegisteredVersion;
		UE_AUTORTFM_OPEN{
			RegisteredVersion = FCurrentCustomVersions::Get(CustomKey);
		};

		checkf(RegisteredVersion, TEXT("Attempted to set a version that is not registered"));

		Versions.Emplace(FCustomVersion(CustomKey, RegisteredVersion->Version, RegisteredVersion->FriendlyName));
	}
}
