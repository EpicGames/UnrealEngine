// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/FileRegions.h"

#include "Misc/AssertionMacros.h"
#include "Serialization/Archive.h"
#include "Serialization/CompactBinary.h"
#include "Serialization/CompactBinaryWriter.h"

const TCHAR* FFileRegion::RegionsFileExtension = TEXT(".uregs");

// NOTE: This serialization function must match FileRegion::ReadRegionsFromFile in AutomationUtils/FileRegions.cs
inline FArchive& operator<<(FArchive& Ar, FFileRegion& Region)
{
	Ar << Region.Offset;
	Ar << Region.Length;
	Ar << Region.Type;
	return Ar;
}

FCbWriter& operator<<(FCbWriter& Writer, const FFileRegion& Region)
{
	Writer.BeginObject();

	Writer << "offset" << Region.Offset;
	Writer << "length" << Region.Length;
	Writer << "type" << static_cast<uint8>(Region.Type);

	Writer.EndObject();

	return Writer;
}

bool FFileRegion::LoadFromCompactBinary(FCbFieldView Obj, FFileRegion& OutRegion)
{
	OutRegion.Offset = Obj["offset"].AsUInt64();
	OutRegion.Length = Obj["length"].AsUInt64();
	OutRegion.Type = static_cast<EFileRegionType>(Obj["type"].AsUInt8());

	return true;
}

// NOTE: This serialization function must match FileRegion::ReadRegionsFromFile in AutomationUtils/FileRegions.cs
void FFileRegion::SerializeFileRegions(class FArchive& Ar, TArray<FFileRegion>& Regions)
{
	Ar << Regions;
}

void FFileRegion::AccumulateFileRegions(TArray<FFileRegion>& InOutRegions, int64 EntryOffset, int64 PayloadOffset, int64 EndOffset, TArrayView<const FFileRegion> InInnerFileRegions)
{
	// Adjust the offsets in the InnerFileRegions array to match the required payload position.
	TArray<FFileRegion> InnerFileRegions(InInnerFileRegions);
	for (FFileRegion& Region : InnerFileRegions)
	{
		Region.Offset += PayloadOffset;
	}

	int32 LastRegionIndex = -1;
	auto AppendRegion = [&](FFileRegion NewRegion)
	{
		if (LastRegionIndex == -1 || InOutRegions[LastRegionIndex].Type != NewRegion.Type)
		{
			// First region in this file, or the type changed.
			LastRegionIndex = InOutRegions.Add(NewRegion);
		}
		else
		{
			// Merge contiguous regions with the same type into a single region.
			FFileRegion& PrevRegion = InOutRegions[LastRegionIndex];
			check(NewRegion.Offset == PrevRegion.Offset + PrevRegion.Length); // Regions must be contiguous
			PrevRegion.Length += NewRegion.Length;
		}
	};

	int64 CurrentOffset = EntryOffset;
	for (int32 Index = 0; Index < InnerFileRegions.Num(); ++Index)
	{
		FFileRegion& CurrentRegion = InnerFileRegions[Index];
		if (CurrentRegion.Length == 0)
			continue; // Skip empty regions

		if (CurrentOffset < int64(CurrentRegion.Offset))
		{
			// Fill the gap with a none-type region
			AppendRegion(FFileRegion(CurrentOffset, CurrentRegion.Offset - CurrentOffset, EFileRegionType::None));
		}

		AppendRegion(CurrentRegion);

		CurrentOffset = CurrentRegion.Offset + CurrentRegion.Length;
	}

	if (CurrentOffset < EndOffset)
	{
		// Add a final none-type region for any remaining data.
		AppendRegion(FFileRegion(CurrentOffset, EndOffset - CurrentOffset, EFileRegionType::None));
	}
}
