// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/Formatters/BinaryArchiveFormatter.h"

FBinaryArchiveFormatter::FBinaryArchiveFormatter(FArchive& InInner) 
	: Inner(InInner)
{
}

FBinaryArchiveFormatter::~FBinaryArchiveFormatter()
{
}

bool FBinaryArchiveFormatter::HasDocumentTree() const
{
	return false;
}
