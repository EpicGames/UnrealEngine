// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/LargeMemoryData.h"

#include "AutoRTFM.h"
#include "Containers/LockFreeList.h"
#include "HAL/IConsoleManager.h"
#include "HAL/UnrealMemory.h"
#include "Math/UnrealMathUtility.h"


int32 GPooledLargeMemoryData_MaxPoolLength = 2;
static FAutoConsoleVariableRef CPooledLargeMemoryData_MaxPoolLength(
	TEXT("s.LargeMemoryDataMaxPoolLength"),
	GPooledLargeMemoryData_MaxPoolLength,
	TEXT("Limit LargeMemoryData pool size to the given number of elements."),
	ECVF_Default

);
static constexpr int64 GPooledLargeMemoryData_MaxElementSize = 128 * 1024;

/*----------------------------------------------------------------------------
	FLargeMemoryData
----------------------------------------------------------------------------*/

FLargeMemoryData::FLargeMemoryData(const int64 PreAllocateBytes)
	: Data(nullptr)
	, NumBytes(FMath::Max<int64>(0, PreAllocateBytes))
	, MaxBytes(0)
{
	GrowBuffer();
}

FLargeMemoryData::~FLargeMemoryData()
{
	if (Data)
	{
		FMemory::Free(Data);
	}
}

bool FLargeMemoryData::Write(void* InData, int64 InOffset, int64 InNum)
{
	// Allow InData to be null if InNum == 0.
	if (InOffset >= 0 && (InData ? InNum >= 0 : InNum == 0))
	{
		// Grow the buffer to the offset position even if InNum == 0.
		NumBytes = FMath::Max<int64>(NumBytes, InOffset + InNum);
		if (NumBytes > MaxBytes)
		{
			GrowBuffer();
		}

		if (InNum)
		{
			FMemory::Memcpy(&Data[InOffset], InData, InNum);
		}

		return true;
	}
	else
	{
		return false;
	}
}

bool FLargeMemoryData::Read(void* OutData, int64 InOffset, int64 InNum) const
{
	// Allow OutData to be null if InNum == 0.
	if (InOffset >= 0 && (OutData ? InNum >= 0 : InNum == 0) && (InOffset + InNum <= NumBytes))
	{
		if (InNum)
		{
			FMemory::Memcpy(OutData, &Data[InOffset], InNum);
		}

		return true;
	}
	else
	{
		return false;
	}
}

uint8* FLargeMemoryData::ReleaseOwnership()
{
	uint8* ReturnData = Data;

	Data = nullptr;
	NumBytes = 0;
	MaxBytes = 0;

	return ReturnData;
}

void FLargeMemoryData::Reserve(int64 NewMax)
{
	if (MaxBytes < NewMax)
	{
		// Allocate slack proportional to the buffer size. Min 64 KB
		MaxBytes = NewMax;

		Data = (uint8*)FMemory::Realloc(Data, MaxBytes);
	}
}

void FLargeMemoryData::GrowBuffer()
{
	// Allocate slack proportional to the buffer size. Min 64 KB
	MaxBytes = FMath::Max<int64>(64 * 1024, FMemory::QuantizeSize(NumBytes + 3 * NumBytes / 8 + 16));

	if (Data)
	{
		Data = (uint8*)FMemory::Realloc(Data, MaxBytes);
	}
	else
	{
		Data = (uint8*)FMemory::Malloc(MaxBytes);
	}
}

/*----------------------------------------------------------------------------
	FPooledLargeMemoryData
----------------------------------------------------------------------------*/

TLockFreePointerListUnordered<FLargeMemoryData, 0> FPooledLargeMemoryData::FreeList;
std::atomic<int32> FPooledLargeMemoryData::FreeListLength;

FPooledLargeMemoryData::FPooledLargeMemoryData()
{
	AutoRTFM::Open([&]
	{
		Data = FreeList.Pop();
		if (Data != nullptr)
		{
			--FreeListLength;
		}
		else
		{
			Data = new FLargeMemoryData();
		}
	});
	AutoRTFM::OnAbort([Data = this->Data] { Free(Data); });
}

FPooledLargeMemoryData::~FPooledLargeMemoryData()
{
	// If outside of a closed transaction, immediately free the FLargeMemoryData.
	// If inside a closed transaction, then defer the free until the transaction
	// completes as writes to the memory may be undone if the transaction aborts.
	AutoRTFM::OnCommit([Data = this->Data] { Free(Data); });
}

void FPooledLargeMemoryData::Free(FLargeMemoryData* Data)
{
	if (Data->GetSize() > GPooledLargeMemoryData_MaxElementSize)
	{
		delete Data;
	}
	else
	{
		if (FreeListLength.fetch_add(1) >= GPooledLargeMemoryData_MaxPoolLength)
		{
			// There's a chance of a race condition here if removal and addition are done at the same time.
			// The outcome of that would be unnecessary destruction of Data that could be pooled,
			// which is considered OK for the sake of simplicity.
			--FreeListLength;
			delete Data;
		}
		else
		{
			FreeList.Push(Data);
		}
	}
}

