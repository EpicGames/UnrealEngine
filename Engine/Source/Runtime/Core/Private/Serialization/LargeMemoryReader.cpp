// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/LargeMemoryReader.h"

#include "CoreGlobals.h"
#include "HAL/UnrealMemory.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Trace/Detail/Channel.h"

/*----------------------------------------------------------------------------
	FLargeMemoryReader
----------------------------------------------------------------------------*/

FLargeMemoryReader::FLargeMemoryReader(const uint8* InData, const int64 Num, ELargeMemoryReaderFlags InFlags, const FName InArchiveName)
	: FMemoryArchive()
	, bFreeOnClose((InFlags & ELargeMemoryReaderFlags::TakeOwnership) != ELargeMemoryReaderFlags::None)
	, Data(InData)
	, NumBytes(Num)
	, ArchiveName(InArchiveName)
{
	UE_CLOGF(!(InData && Num > 0), LogSerialization, Fatal, "Tried to initialize an FLargeMemoryReader with a null or empty buffer. Archive name: %ls.", *ArchiveName.ToString());
	this->SetIsLoading(true);
	this->SetIsPersistent((InFlags & ELargeMemoryReaderFlags::Persistent) != ELargeMemoryReaderFlags::None);
}

void FLargeMemoryReader::Serialize(void* OutData, int64 Num)
{
	if (Num && !IsError())
	{
		// Only serialize if we have the requested amount of data
		if (Offset + Num <= NumBytes)
		{
			FMemory::Memcpy(OutData, &Data[Offset], Num);
			Offset += Num;
		}
		else
		{
			SetError();
		}
	}
}

int64 FLargeMemoryReader::TotalSize()
{
	return NumBytes;
}

FString FLargeMemoryReader::GetArchiveName() const
{
	return ArchiveName.ToString();
}

FLargeMemoryReader::~FLargeMemoryReader()
{
	if (bFreeOnClose)
	{
		FMemory::Free((void*)Data);
	}
}
