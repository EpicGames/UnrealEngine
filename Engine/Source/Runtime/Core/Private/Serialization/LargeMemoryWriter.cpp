// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/LargeMemoryWriter.h"

#include "CoreGlobals.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Trace/Detail/Channel.h"

/*----------------------------------------------------------------------------
	FLargeMemoryWriter
----------------------------------------------------------------------------*/

FLargeMemoryWriter::FLargeMemoryWriter(const int64 PreAllocateBytes, bool bIsPersistent, const TCHAR* InFilename)
	: FMemoryArchive()
	, Data(PreAllocateBytes)
	, ArchiveName(InFilename ? InFilename : TEXT("FLargeMemoryWriter"))
{
	this->SetIsSaving(true);
	this->SetIsPersistent(bIsPersistent);
}

FLargeMemoryWriter::~FLargeMemoryWriter() = default;

void FLargeMemoryWriter::Serialize(void* InData, int64 Num)
{
	UE_CLOGF(!Data.HasData(), LogSerialization, Fatal, "Tried to serialize data to an FLargeMemoryWriter that was already released. Archive name: %ls.", *ArchiveName);

	if (Data.Write(InData, Offset, Num))
	{
		Offset += Num;
	}
}

FString FLargeMemoryWriter::GetArchiveName() const
{
	return ArchiveName;
}

uint8* FLargeMemoryWriter::GetData() const
{
	UE_CLOGF(!Data.HasData(), LogSerialization, Warning, "Tried to get written data from an FLargeMemoryWriter that was already released. Archive name: %ls.", *ArchiveName);
	return const_cast<uint8*>(Data.GetData());
}
