// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/LoadTimeTrace.h"

#if LOADTIMEPROFILERTRACE_ENABLED

#include "Trace/Trace.inl"
#include "Misc/CString.h"
#include "HAL/PlatformTLS.h"

UE_TRACE_CHANNEL_DEFINE(LoadTimeChannel, "Load time profiling for packages and request groups. Can be viewed in Asset Loading Insights.")
UE_TRACE_CHANNEL_DEFINE(AssetLoadTimeChannel, "Asset-specific load time profiling and timing. This channel is only used as a toggle for CPU Profiler scoped events related to asset loading.")

UE_TRACE_EVENT_BEGIN(LoadTime, BeginRequestGroup)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, Format)
	UE_TRACE_EVENT_FIELD(uint8[], FormatArgs)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(LoadTime, EndRequestGroup)
UE_TRACE_EVENT_END()

FLoadTimeProfilerTrace::FRequestGroupScope::~FRequestGroupScope()
{
	UE_TRACE_LOG(LoadTime, EndRequestGroup, LoadTimeChannel);
}

void FLoadTimeProfilerTrace::FRequestGroupScope::OutputBegin()
{
	UE_TRACE_LOG(LoadTime, BeginRequestGroup, LoadTimeChannel)
		<< BeginRequestGroup.Format(FormatString)
		<< BeginRequestGroup.FormatArgs(FormatArgsBuffer, FormatArgsSize);
}


#endif

