// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/MappedName.h"

#include "Serialization/Archive.h"
#include "UObject/NameBatchSerialization.h"

FArchive& operator<<(FArchive& Ar, FMappedName& MappedName)
{
	Ar << MappedName.Index << MappedName.Number;

	return Ar;
}

void FNameMap::Load(FArchive& Ar, FMappedName::EType InNameMapType)
{
	NameEntries = LoadNameBatch(Ar);
	NameMapType = InNameMapType;
}
