// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/MemoryImage.h"

#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include "Misc/SecureHash.h"
#include "Misc/StringBuilder.h"
#include "Misc/MemStack.h"
#include "Interfaces/ITargetPlatform.h"
#include "UObject/NameTypes.h"
#include "Hash/CityHash.h"
#include "ProfilingDebugging/LoadTimeTracker.h"
#include "Misc/ScopeRWLock.h"
#include "Misc/DataDrivenPlatformInfoRegistry.h"
#include "Serialization/Archive.h"
#include "Serialization/ShaderKeyGenerator.h"
#include "Async/ParallelFor.h"
#if WITH_EDITOR
#include "Serialization/CompactBinary.h"
#include "Serialization/CompactBinaryWriter.h"
#endif

DEFINE_LOG_CATEGORY_STATIC(LogMemoryImage, Log, All);

IMPLEMENT_TYPE_LAYOUT(FMemoryImageString);
IMPLEMENT_TYPE_LAYOUT(FPlatformTypeLayoutParameters);
IMPLEMENT_TYPE_LAYOUT(FHashedName);

// Guid for versioning format changes/bug fixes to memory images
#define MEMORYIMAGE_DERIVEDDATA_VER_ANSI		"B6A0A753ED62499D8F30931E61FCAB05"

static const uint32 NumTypeLayoutDescHashBuckets = 4357u;
static const FTypeLayoutDesc* GTypeLayoutHashBuckets[NumTypeLayoutDescHashBuckets] = { nullptr };
static uint32 GNumTypeLayoutsRegistered = 0u;

bool FPlatformTypeLayoutParameters::IsCurrentPlatform() const
{
	FPlatformTypeLayoutParameters CurrentPlatform;
	CurrentPlatform.InitializeForCurrent();
	return *this == CurrentPlatform;
}

void FPlatformTypeLayoutParameters::InitializeForArchive(FArchive& Ar)
{
	check(Ar.IsSaving());
	if (Ar.IsCooking())
	{
		InitializeForPlatform(Ar.CookingTarget());
	}
	else
	{
		InitializeForCurrent();
	}
}

void FPlatformTypeLayoutParameters::InitializeForPlatform(const ITargetPlatform* TargetPlatform)
{
	if (TargetPlatform)
	{
		InitializeForPlatform(TargetPlatform->IniPlatformName(), TargetPlatform->HasEditorOnlyData());
	}
	else
	{
		InitializeForCurrent();
	}
}

void FPlatformTypeLayoutParameters::InitializeForPlatform(const FString& PlatformName, bool bHasEditorOnlyData)
{
	const FDataDrivenPlatformInfo& PlatformInfo = FDataDrivenPlatformInfoRegistry::GetPlatformInfo(PlatformName);

	Flags = Flag_Initialized;
	if (bHasEditorOnlyData) Flags |= Flag_WithEditorOnly;
	if (PlatformInfo.Freezing_b32Bit) Flags |= Flag_Is32Bit;
	if (PlatformInfo.Freezing_bAlignBases) Flags |= Flag_AlignBases;

	MaxFieldAlignment = PlatformInfo.Freezing_MaxFieldAlignment;
}

void FPlatformTypeLayoutParameters::InitializeForCurrent()
{
	Flags = Flag_Initialized;
	if (WITH_EDITORONLY_DATA) Flags |= Flag_WithEditorOnly;

	check(GetRawPointerSize() == sizeof(void*));

	// clang for MS matches the MSVC ABI
#if defined(__clang__) && !PLATFORM_MICROSOFT
	InitializeForClang();
#else
	InitializeForMSVC();
#endif
}

void FPlatformTypeLayoutParameters::InitializeForMSVC()
{
	Flags |= Flag_AlignBases;

	// This corresponds to the value used by /Zp#
	MaxFieldAlignment = Is32Bit() ? 4u : 8u;
}

void FPlatformTypeLayoutParameters::InitializeForClang()
{
	// nothing
}

FArchive& FPlatformTypeLayoutParameters::Serialize(FArchive& Ar)
{
	// if you change this code, please bump MATERIALSHADERMAP_DERIVEDDATA_VER (see FMaterialShaderMap::Serialize)
	// since this is a part of ShaderMapId
	return Ar << MaxFieldAlignment << Flags;
}

void FPlatformTypeLayoutParameters::AppendKeyString(FString& KeyString) const
{
	FShaderKeyGenerator KeyGen(KeyString);
	Append(KeyGen);
}

void FPlatformTypeLayoutParameters::Append(FShaderKeyGenerator& KeyGen) const
{
	if (KeyGen.IsBinary())
	{
		KeyGen.BinaryAppend(&Flags, sizeof(Flags));
		KeyGen.BinaryAppend(&MaxFieldAlignment, sizeof(MaxFieldAlignment));
		FAnsiStringView VersionText(MEMORYIMAGE_DERIVEDDATA_VER_ANSI);
		KeyGen.BinaryAppend(VersionText.GetData(), VersionText.Len() * sizeof(VersionText[0]));
	}
	else
	{
		KeyGen.TextGetResultString() +=
			FString::Printf(TEXT("FL_%08x_MFA_%08x_V_" MEMORYIMAGE_DERIVEDDATA_VER_ANSI), Flags, MaxFieldAlignment);
	}
}

#if WITH_EDITOR
void FPlatformTypeLayoutParameters::Save(FCbWriter& Writer) const
{
	Writer.BeginArray();
	Writer << MaxFieldAlignment;
	Writer << Flags;
	Writer.EndArray();
}

bool FPlatformTypeLayoutParameters::TryLoad(FCbFieldView Field)
{
	*this = FPlatformTypeLayoutParameters();

	FCbFieldViewIterator ElementField(Field.CreateViewIterator());
	MaxFieldAlignment = ElementField.AsUInt32();
	if ((ElementField++).HasError()) return false;
	Flags = ElementField.AsUInt32();
	if ((ElementField++).HasError()) return false;
	return true;
}

bool LoadFromCompactBinary(FCbFieldView Field, FPlatformTypeLayoutParameters& OutValue)
{
	return OutValue.TryLoad(Field);
}

#endif

// evaluated during static-initialization, so logging from regular check() macros won't work correctly
static void InitializeSizeFromFields(FTypeLayoutDesc& TypeLayout, const FPlatformTypeLayoutParameters& PlatformLayoutParams)
{
	check(!TypeLayout.IsIntrinsic);
	check(TypeLayout.SizeFromFields == ~0u);

	const FFieldLayoutDesc* Field = TypeLayout.Fields;
	if (!Field && !ETypeLayoutInterface::HasVTable(TypeLayout.Interface))
	{
		// Empty type
		check(TypeLayout.Size == 1u);
		TypeLayout.SizeFromFields = 0u;
		return;
	}

	const FTypeLayoutDesc* CurrentBitFieldType = nullptr;
	uint32 CurrentNumBits = 0u;
	uint32 Offset = 0u;
	uint32 FieldIndex = 0u;
	uint32 NumEmptyBases = 0u;

	if (ETypeLayoutInterface::HasVTable(TypeLayout.Interface) && TypeLayout.NumVirtualBases == 0u)
	{
		Offset += sizeof(void*);
	}

	while (Field)
	{
		const FTypeLayoutDesc& FieldType = *Field->Type;
		if (Field->BitFieldSize == 0u)
		{
			const bool bIsBase = (FieldIndex < TypeLayout.NumBases);
			if (CurrentBitFieldType)
			{
				Offset = Align(Offset, FMath::Min(CurrentBitFieldType->Alignment, PlatformLayoutParams.MaxFieldAlignment));
				Offset += CurrentBitFieldType->Size;
				CurrentBitFieldType = nullptr;
				CurrentNumBits = 0u;
			}

			const uint32 FieldTypeAlignment = Freeze::GetTargetAlignment(FieldType, PlatformLayoutParams);
			const uint32 FieldAlignment = FMath::Min(FieldTypeAlignment, PlatformLayoutParams.MaxFieldAlignment);
			uint32 PaddedFieldSize = FieldType.SizeFromFields;

			Offset = Align(Offset, FieldAlignment);
			if (PaddedFieldSize == 0u && bIsBase)
			{
				if (NumEmptyBases > 0u)
				{
					checkf(Offset == Field->Offset, TEXT("[%s::%s] Calculated Offset = %d, Real Offset = %d"), TypeLayout.Name, Field->Name, Offset, Field->Offset);
					PaddedFieldSize = 1u;
				}
				else
				{
					// Empty bases are allowed to have Offset 0, since they logically overlap
					checkf(Offset == Field->Offset || Field->Offset == 0u, TEXT("[%s::%s] Calculated Offset = %d, Real Offset = %d"), TypeLayout.Name, Field->Name, Offset, Field->Offset);
					++NumEmptyBases;
				}
			}
			else
			{
				checkf(Offset == Field->Offset || Field->Offset == 0u, TEXT("[%s::%s] Calculated Offset = %d, Real Offset = %d"), TypeLayout.Name, Field->Name, Offset, Field->Offset);
				if (PaddedFieldSize == 0u)
				{
					PaddedFieldSize = 1u;
				}
			}

			if (PaddedFieldSize > 0u)
			{
				if (!bIsBase || PlatformLayoutParams.HasAlignBases())
				{
					const uint32 FieldSize = Align(PaddedFieldSize, FieldTypeAlignment);
					check(FieldSize == FieldType.Size);
					Offset += FieldSize * Field->NumArray;
				}
				else
				{
					check(Field->NumArray == 1u);
					Offset += PaddedFieldSize;
				}
			}
		}
		else
		{
			if (CurrentBitFieldType == &FieldType && CurrentNumBits + Field->BitFieldSize <= FieldType.Size * 8u)
			{
				CurrentNumBits += Field->BitFieldSize;
				// reuse previous offset
				const_cast<FFieldLayoutDesc*>(Field)->Offset = Offset;
			}
			else
			{
				if (CurrentBitFieldType)
				{
					Offset = Align(Offset, FMath::Min(CurrentBitFieldType->Alignment, PlatformLayoutParams.MaxFieldAlignment));
					Offset += CurrentBitFieldType->Size;
				}
				check(FieldType.Size <= sizeof(uint64));
				const_cast<FFieldLayoutDesc*>(Field)->Offset = Offset;
				CurrentBitFieldType = &FieldType;
				CurrentNumBits = Field->BitFieldSize;
			}
		}

		Field = Field->Next;
		++FieldIndex;
	}

	if (CurrentBitFieldType)
	{
		Offset = Align(Offset, FMath::Min(CurrentBitFieldType->Alignment, PlatformLayoutParams.MaxFieldAlignment));
		Offset += CurrentBitFieldType->Size;
	}

	const uint32 AlignedOffset = Align(Offset, FMath::Min(TypeLayout.Alignment, PlatformLayoutParams.MaxFieldAlignment));
	checkf(AlignedOffset == TypeLayout.Size, TEXT("[%s] Calculated Size: %d, Real Size: %d"), TypeLayout.Name, AlignedOffset, TypeLayout.Size);

	TypeLayout.SizeFromFields = Offset;
}

const FTypeLayoutDesc& FTypeLayoutDesc::GetInvalidTypeLayout()
{
	static const FTypeLayoutDesc InvalidTypeDesc = { 0 };
	checkf(false, TEXT("Access to Abstract/Invalid type layout desc"));
	return InvalidTypeDesc;
}

void FTypeLayoutDesc::Initialize(FTypeLayoutDesc& TypeDesc)
{
	FPlatformTypeLayoutParameters PlatformLayoutParams;
	PlatformLayoutParams.InitializeForCurrent();
	InitializeSizeFromFields(TypeDesc, PlatformLayoutParams);
}

void FTypeLayoutDesc::Register(FTypeLayoutDesc& TypeDesc)
{
	const FHashedName HashedName(TypeDesc.Name);
	TypeDesc.NameHash = HashedName.GetHash();

	const uint32 BucketIndex = (uint32)(TypeDesc.NameHash % NumTypeLayoutDescHashBuckets);
	TypeDesc.HashNext = GTypeLayoutHashBuckets[BucketIndex];
	GTypeLayoutHashBuckets[BucketIndex] = &TypeDesc;
	++GNumTypeLayoutsRegistered;
}

const FTypeLayoutDesc* FTypeLayoutDesc::Find(uint64 NameHash)
{
	SCOPED_LOADTIMER(FTypeLayoutDesc_Find);

	const uint32 BucketIndex = NameHash % NumTypeLayoutDescHashBuckets;
	const FTypeLayoutDesc* TypeDesc = GTypeLayoutHashBuckets[BucketIndex];
	while (TypeDesc)
	{
		if (TypeDesc->NameHash == NameHash)
		{
			return TypeDesc;
		}
		TypeDesc = TypeDesc->HashNext;
	}
	return nullptr;
}

void InternalDeleteObjectFromLayout(void* Object, const FTypeLayoutDesc& TypeDesc, const FPointerTableBase* PtrTable, bool bIsFrozen)
{
	check(Object);
	// DestroyFunc may be nullptr for types with trivial destructors
	if (TypeDesc.DestroyFunc)
	{
		TypeDesc.DestroyFunc(Object, TypeDesc, PtrTable, bIsFrozen);
	}
	if (!bIsFrozen)
	{
		::operator delete(Object);
	}
}

static bool TryGetOffsetToBase(const FTypeLayoutDesc& TypeDesc, const FTypeLayoutDesc& BaseTypeDesc, uint32& OutOffset)
{
	if (TypeDesc == BaseTypeDesc)
	{
		OutOffset = 0u;
		return true;
	}

	const FFieldLayoutDesc* Field = TypeDesc.Fields;
	for (uint32 BaseIndex = 0u; BaseIndex < TypeDesc.NumBases; ++BaseIndex)
	{
		check(Field);

		uint32 FieldOffsetToBase = 0u;
		if (TryGetOffsetToBase(*Field->Type, BaseTypeDesc, FieldOffsetToBase))
		{
			OutOffset = Field->Offset + FieldOffsetToBase;
			return true;
		}
		Field = Field->Next;
	}

	return false;
}

uint32 FTypeLayoutDesc::GetOffsetToBase(const FTypeLayoutDesc& BaseTypeDesc) const
{
	uint32 Offset = 0u;
	const bool bFound = TryGetOffsetToBase(*this, BaseTypeDesc, Offset);
	check(bFound);
	return Offset;
}

bool FTypeLayoutDesc::IsDerivedFrom(const FTypeLayoutDesc& BaseTypeDesc) const
{
	uint32 UnusedOffset = 0u;
	return TryGetOffsetToBase(*this, BaseTypeDesc, UnusedOffset);
}

void Freeze::ExtractBitFieldValue(const void* Value, uint32 SrcBitOffset, uint32 DestBitOffset, uint32 NumBits, uint64& InOutValue)
{
	for (uint32 SrcBitIndex = SrcBitOffset,DestBitIndex = DestBitOffset; SrcBitIndex < SrcBitOffset + NumBits; ++SrcBitIndex,++DestBitIndex)
	{
		const uint32 SrcByteOffset = SrcBitIndex / 8u;
		const uint32 SrcBitOffsetInByte = SrcBitIndex & 7u;
		const uint8* SrcByte = (uint8*)Value + SrcByteOffset;
		const uint64 SrcBitValue = (*SrcByte >> SrcBitOffsetInByte) & 1u;

		InOutValue |= (SrcBitValue << DestBitIndex);
	}
}

bool Freeze::IncludeField(const FFieldLayoutDesc* FieldDesc, const FPlatformTypeLayoutParameters& LayoutParams)
{
	const bool bIsEditorOnly = (FieldDesc->Flags & EFieldLayoutFlags::WithEditorOnly) != 0u;

	if (bIsEditorOnly && !LayoutParams.WithEditorOnly())
	{
		return false;
	}

	return true;
}

uint32 Freeze::GetTargetAlignment(const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams)
{
	return TypeDesc.GetTargetAlignmentFunc(TypeDesc, LayoutParams);
}

void Freeze::DefaultWriteMemoryImageField(FMemoryImageWriter& Writer, const void* Object, const void* FieldObject, const FTypeLayoutDesc& TypeDesc, const FTypeLayoutDesc& DerivedTypeDesc)
{
	TypeDesc.WriteFrozenMemoryImageFunc(Writer, FieldObject, TypeDesc, DerivedTypeDesc);
}

void Freeze::DefaultWriteMemoryImage(FMemoryImageWriter& Writer, const void* Object, const FTypeLayoutDesc& TypeDesc, const FTypeLayoutDesc& DerivedTypeDesc)
{
	const FPlatformTypeLayoutParameters& TargetLayoutParams = Writer.GetTargetLayoutParams();
	// VTable will be shared with any base class vtable, if present
	if (ETypeLayoutInterface::HasVTable(TypeDesc.Interface) && TypeDesc.NumVirtualBases == 0u)
	{
		Writer.WriteVTable(TypeDesc, DerivedTypeDesc);
	}

	const FTypeLayoutDesc* CurrentSrcBitFieldType = nullptr;
	const FTypeLayoutDesc* CurrentDestBitFieldType = nullptr;
	uint64 CurrentBitFieldValue = 0u;
	uint32 CurrentBitFieldOffset = 0u;
	uint32 CurrentSrcNumBits = 0u;
	uint32 CurrentDestNumBits = 0u;

	const FFieldLayoutDesc* FieldDesc = TypeDesc.Fields;
	const uint32 NumBases = TypeDesc.NumBases;
	uint32 FieldIndex = 0u;
	uint32 NumEmptyBases = 0u;

	while (FieldDesc)
	{
		const bool bIsBase = (FieldIndex < NumBases);
		const FTypeLayoutDesc& FieldType = *FieldDesc->Type;

		bool bIncludeField = IncludeField(FieldDesc, TargetLayoutParams);

		if (FieldDesc->BitFieldSize > 0)
		{
			// reset src bitfield if necessary
			if (CurrentSrcBitFieldType != &FieldType || CurrentSrcNumBits + FieldDesc->BitFieldSize > FieldType.Size * 8u)
			{
				CurrentSrcBitFieldType = &FieldType;
				CurrentSrcNumBits = 0u;
			}
		}

		if (bIncludeField)
		{
			const uint8* FieldObject = (uint8*)Object + FieldDesc->Offset;

			if (FieldDesc->BitFieldSize == 0u)
			{
				if (CurrentDestBitFieldType)
				{
					if (CurrentDestNumBits > 0)
					{
						Writer.WriteAlignment(FMath::Min(CurrentDestBitFieldType->Alignment, TargetLayoutParams.MaxFieldAlignment));
						Writer.WriteBytes(&CurrentBitFieldValue, CurrentDestBitFieldType->Size);
					}
					CurrentDestBitFieldType = nullptr;
					CurrentDestNumBits = 0u;
					CurrentBitFieldValue = 0u;
				}

				uint32 PaddedFieldSize = FieldType.SizeFromFields;
				if (PaddedFieldSize == 0u && bIsBase)
				{
					if (NumEmptyBases > 0u)
					{
						PaddedFieldSize = 1u;
					}
					else
					{
						++NumEmptyBases;
					}
				}
				else
				{
					if (PaddedFieldSize == 0u)
					{
						PaddedFieldSize = 1u;
					}
				}

				if (PaddedFieldSize > 0u)
				{
					const uint32 FieldTypeAlignment = GetTargetAlignment(FieldType, TargetLayoutParams);
					const uint32 FieldAlignment = FMath::Min(FieldTypeAlignment, TargetLayoutParams.MaxFieldAlignment);
					Writer.WriteAlignment(FieldAlignment);
					{
						FFieldLayoutDesc::FWriteFrozenMemoryImageFunc* WriteFieldFunc = FieldDesc->WriteFrozenMemoryImageFunc;
						for (uint32 ArrayIndex = 0u; ArrayIndex < FieldDesc->NumArray; ++ArrayIndex)
						{
							const uint32 FieldOffset = Writer.GetOffset();
							WriteFieldFunc(Writer,
								Object,
								FieldObject + ArrayIndex * FieldType.Size,
								FieldType, bIsBase ? DerivedTypeDesc : FieldType);
							if (!bIsBase || TargetLayoutParams.HasAlignBases())
							{
								// Align the field size
								const uint32 FieldSize = Writer.GetOffset() - FieldOffset;
								Writer.WritePaddingToSize(FieldOffset + Align(FieldSize, FieldTypeAlignment));
							}
						}
					}
				}
			}
			else
			{
				// if we have run out of bits, then we need to move to next byte
				if (CurrentDestBitFieldType != &FieldType || CurrentDestNumBits + FieldDesc->BitFieldSize > FieldType.Size * 8u)
				{
					if (CurrentDestBitFieldType)
					{
						Writer.WriteAlignment(FMath::Min(CurrentDestBitFieldType->Alignment, TargetLayoutParams.MaxFieldAlignment));
						Writer.WriteBytes(&CurrentBitFieldValue, CurrentDestBitFieldType->Size);
					}

					CurrentBitFieldValue = 0u;
					CurrentDestNumBits = 0;
					CurrentDestBitFieldType = &FieldType;
				}

				ExtractBitFieldValue(FieldObject, CurrentSrcNumBits, CurrentDestNumBits, FieldDesc->BitFieldSize, CurrentBitFieldValue);
				CurrentDestNumBits += FieldDesc->BitFieldSize;
			}
		}

		CurrentSrcNumBits += FieldDesc->BitFieldSize;

		FieldDesc = FieldDesc->Next;
		++FieldIndex;
	}

	if (CurrentDestBitFieldType)
	{
		Writer.WriteAlignment(FMath::Min(CurrentDestBitFieldType->Alignment, TargetLayoutParams.MaxFieldAlignment));
		Writer.WriteBytes(&CurrentBitFieldValue, CurrentDestBitFieldType->Size);
	}
}

static bool CheckOffsetMatch(const uint32 CheckOffset, uint32 FieldOffset, const FTypeLayoutDesc& FieldType, bool bIsBase)
{
	if (CheckOffset == FieldOffset)
	{
		return true;
	}
	if (bIsBase && FieldType.SizeFromFields == 0u && FieldOffset == 0u)
	{
		// Empty  bases may have offset of 0, depending on compiler
		// True on clang, false on MSVC, probably depends on how compiler handles multiple empty base classes
		// May need to encode this into FPlatformTypeLayoutParameters at some point
		return true;
	}
	return false;
}

uint32 Freeze::AppendHashForNameAndSize(const TCHAR* Name, uint32 Size, FSHA1& Hasher)
{
	Hasher.UpdateWithString(Name, FCString::Strlen(Name));
	Hasher.Update((uint8*)&Size, sizeof(Size));
	return Size;
}

uint32 Freeze::DefaultAppendHash(const FTypeLayoutDesc& TypeLayout, const FPlatformTypeLayoutParameters& LayoutParams, FSHA1& Hasher)
{
	Hasher.UpdateWithString(TypeLayout.Name, FCString::Strlen(TypeLayout.Name));

	const FFieldLayoutDesc* Field = TypeLayout.Fields;
	if (!Field)
	{
		// Assume size can't change for intrinsic/empty types
		Hasher.Update((uint8*)&TypeLayout.SizeFromFields, sizeof(TypeLayout.SizeFromFields));
		return TypeLayout.SizeFromFields;
	}

	const FTypeLayoutDesc* CurrentBitFieldType = nullptr;
	uint32 CurrentNumBits = 0u;
	uint32 Offset = 0u;
	uint32 FieldIndex = 0u;
	uint32 NumEmptyBases = 0u;

	if (ETypeLayoutInterface::HasVTable(TypeLayout.Interface) && TypeLayout.NumVirtualBases == 0u)
	{
		Offset += LayoutParams.GetRawPointerSize();
	}

	while (Field)
	{
		if (IncludeField(Field, LayoutParams))
		{
			const FTypeLayoutDesc& FieldType = *Field->Type;
			if (Field->BitFieldSize == 0u)
			{
				CurrentBitFieldType = nullptr;
				CurrentNumBits = 0u;

				const bool bIsBase = (FieldIndex < TypeLayout.NumBases);
				const uint32 FieldTypeAlignment = GetTargetAlignment(FieldType, LayoutParams);
				const uint32 FieldAlignment = FMath::Min(FieldTypeAlignment, LayoutParams.MaxFieldAlignment);
				check(!LayoutParams.IsCurrentPlatform() || FieldTypeAlignment == FieldType.Alignment);

				Offset = Align(Offset, FieldAlignment);
				check(!LayoutParams.IsCurrentPlatform() || CheckOffsetMatch(Offset, Field->Offset, FieldType, bIsBase));

				Hasher.Update((uint8*)&Offset, sizeof(Offset));
				Hasher.Update((uint8*)&Field->NumArray, sizeof(Field->NumArray));

				uint32 PaddedFieldSize = FieldType.AppendHashFunc(FieldType, LayoutParams, Hasher);
				
				if (PaddedFieldSize == 0u && bIsBase)
				{
					if (NumEmptyBases > 0u)
					{
						PaddedFieldSize = 1u;
					}
					else
					{
						// Empty bases are allowed to have Offset 0, since they logically overlap
						++NumEmptyBases;
					}
				}
				else
				{
					if (PaddedFieldSize == 0u)
					{
						PaddedFieldSize = 1u;
					}
				}

				if (PaddedFieldSize > 0u)
				{
					if (!bIsBase || LayoutParams.HasAlignBases())
					{
						const uint32 FieldSize = Align(PaddedFieldSize, FieldTypeAlignment);
						Offset += FieldSize * Field->NumArray;
					}
					else
					{
						check(Field->NumArray == 1u);
						Offset += PaddedFieldSize;
					}
				}
			}
			else
			{
				if (CurrentBitFieldType == &FieldType && CurrentNumBits + Field->BitFieldSize <= FieldType.Size * 8u)
				{
					CurrentNumBits += Field->BitFieldSize;
				}
				else
				{
					const uint32 FieldTypeAlignment = GetTargetAlignment(FieldType, LayoutParams);
					const uint32 ClampedFieldAlignment = FMath::Min(FieldTypeAlignment, LayoutParams.MaxFieldAlignment);
					Offset = Align(Offset, ClampedFieldAlignment);
					Hasher.Update((uint8*)&Offset, sizeof(Offset));
					const uint32 FieldSize = FieldType.AppendHashFunc(FieldType, LayoutParams, Hasher);

					check(FieldType.Size <= sizeof(uint64));
					CurrentBitFieldType = &FieldType;
					CurrentNumBits = Field->BitFieldSize;
					Offset += FieldSize;
				}

				Hasher.Update((uint8*)&Field->BitFieldSize, sizeof(Field->BitFieldSize));
			}			
		}

		++FieldIndex;
		Field = Field->Next;
	}

	check(!LayoutParams.IsCurrentPlatform() || Offset == TypeLayout.SizeFromFields);
	return Offset;
}

uint32 Freeze::DefaultGetTargetAlignment(const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams)
{
	uint32 Alignment = 1u;
	if (ETypeLayoutInterface::HasVTable(TypeDesc.Interface))
	{
		Alignment = FMath::Min(LayoutParams.GetRawPointerSize(), LayoutParams.MaxFieldAlignment);
	}

	if (Alignment < LayoutParams.MaxFieldAlignment)
	{
		for (const FFieldLayoutDesc* Field = TypeDesc.Fields; Field != nullptr; Field = Field->Next)
		{
			if (IncludeField(Field, LayoutParams))
			{
				const uint32 FieldTypeAlignment = GetTargetAlignment(*Field->Type, LayoutParams);
				if (FieldTypeAlignment >= LayoutParams.MaxFieldAlignment)
				{
					Alignment = LayoutParams.MaxFieldAlignment;
					break;
				}
				else
				{
					Alignment = FMath::Max(Alignment, FieldTypeAlignment);
				}
			}
		}
	}

	return Alignment;
}

void FMemoryToStringContext::AppendNullptr()
{
	String->Append(TEXT("nullptr\n"));
}

void FMemoryToStringContext::AppendUnfrozenPointer(const FTypeLayoutDesc& StaticTypeDesc)
{
	String->Appendf(TEXT("TMemoryImagePtr<%s> (Unfrozen), Target: "), StaticTypeDesc.Name);
}

void FMemoryToStringContext::AppendFrozenPointer(const FTypeLayoutDesc& StaticTypeDesc, int32 FrozenTypeIndex)
{
	String->Appendf(TEXT("TMemoryImagePtr<%s>, FrozenTypeIndex: %d, Target: "), StaticTypeDesc.Name, FrozenTypeIndex);
}

void FMemoryToStringContext::AppendIndent()
{
	for (int32 i = 0; i < Indent; ++i)
	{
		String->Append(TEXT("    "));
	}
}

void Freeze::DefaultToString(const void* Object, const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams, FMemoryToStringContext& OutContext)
{
	OutContext.String->Appendf(TEXT("%s\n"), TypeDesc.Name);
	++OutContext.Indent;

	const FFieldLayoutDesc* FieldDesc = TypeDesc.Fields;
	while (FieldDesc)
	{
		if (Freeze::IncludeField(FieldDesc, LayoutParams))
		{
			const FTypeLayoutDesc& FieldType = *FieldDesc->Type;
			const uint8* FieldObject = (uint8*)Object + FieldDesc->Offset;

			OutContext.AppendIndent();
			OutContext.String->Appendf(TEXT("%s: "), FieldDesc->Name);

			if (FieldDesc->BitFieldSize == 0u)
			{
				FieldType.ToStringFunc(FieldObject, FieldType, LayoutParams, OutContext);
			}
			else
			{
				OutContext.String->Append(TEXT("(BITFIELD)\n"));
			}
		}

		FieldDesc = FieldDesc->Next;
	}

	--OutContext.Indent;
}

uint32 Freeze::IntrinsicAppendHash(void* const* DummyObject, const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams, FSHA1& Hasher)
{
	return AppendHashForNameAndSize(TypeDesc.Name, LayoutParams.GetRawPointerSize(), Hasher);
}

uint32 Freeze::IntrinsicGetTargetAlignment(void* const* DummyObject, const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams)
{
	return LayoutParams.GetRawPointerSize();
}

void Freeze::IntrinsicToString(char Object, const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams, FMemoryToStringContext& OutContext)
{
	OutContext.String->Appendf(TEXT("%d\n"), Object);
}
void Freeze::IntrinsicToString(short Object, const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams, FMemoryToStringContext& OutContext)
{
	OutContext.String->Appendf(TEXT("%d\n"), Object);
}
void Freeze::IntrinsicToString(int Object, const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams, FMemoryToStringContext& OutContext)
{
	OutContext.String->Appendf(TEXT("%d\n"), Object);
}
void Freeze::IntrinsicToString(int8 Object, const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams, FMemoryToStringContext& OutContext)
{
	OutContext.String->Appendf(TEXT("%d\n"), Object);
}
void Freeze::IntrinsicToString(long Object, const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams, FMemoryToStringContext& OutContext)
{
	OutContext.String->Appendf(TEXT("%d\n"), Object);
}
void Freeze::IntrinsicToString(long long Object, const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams, FMemoryToStringContext& OutContext)
{
	OutContext.String->Appendf(TEXT("%d\n"), Object);
}
void Freeze::IntrinsicToString(unsigned char Object, const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams, FMemoryToStringContext& OutContext)
{
	OutContext.String->Appendf(TEXT("%d\n"), Object);
}
void Freeze::IntrinsicToString(unsigned short Object, const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams, FMemoryToStringContext& OutContext)
{
	OutContext.String->Appendf(TEXT("%d\n"), Object);
}
void Freeze::IntrinsicToString(unsigned int Object, const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams, FMemoryToStringContext& OutContext)
{
	OutContext.String->Appendf(TEXT("%d\n"), Object);
}
void Freeze::IntrinsicToString(unsigned long Object, const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams, FMemoryToStringContext& OutContext)
{
	OutContext.String->Appendf(TEXT("%d\n"), Object);
}
void Freeze::IntrinsicToString(unsigned long long Object, const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams, FMemoryToStringContext& OutContext)
{
	OutContext.String->Appendf(TEXT("%d\n"), Object);
}
void Freeze::IntrinsicToString(float Object, const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams, FMemoryToStringContext& OutContext)
{
	OutContext.String->Appendf(TEXT("%g\n"), Object);
}
void Freeze::IntrinsicToString(double Object, const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams, FMemoryToStringContext& OutContext)
{
	OutContext.String->Appendf(TEXT("%g\n"), Object);
}
void Freeze::IntrinsicToString(wchar_t Object, const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams, FMemoryToStringContext& OutContext)
{
	OutContext.String->Appendf(TEXT("%d\n"), Object);
}
void Freeze::IntrinsicToString(char16_t Object, const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams, FMemoryToStringContext& OutContext)
{
	OutContext.String->Appendf(TEXT("%d\n"), Object);
}
void Freeze::IntrinsicToString(void* Object, const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams, FMemoryToStringContext& OutContext)
{
	OutContext.String->Appendf(TEXT("%p\n"), Object);
}
void Freeze::IntrinsicToString(const FHashedName& Object, const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams, FMemoryToStringContext& OutContext)
{
	OutContext.String->Appendf(TEXT("%016llX\n"), Object.GetHash());
}

void FMemoryImageString::ToString(FMemoryToStringContext& OutContext) const
{
	if (Data.Num() > 0)
	{
		OutContext.String->Appendf(TEXT("\"%s\"\n"), Data.GetData());
	}
	else
	{
		OutContext.String->Append(TEXT("\"\"\n"));
	}
}

uint32 Freeze::AppendHash(const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams, FSHA1& Hasher)
{
	return TypeDesc.AppendHashFunc(TypeDesc, LayoutParams, Hasher);
}

uint32 Freeze::AppendHashPair(const FTypeLayoutDesc& KeyTypeDesc, const FTypeLayoutDesc& ValueTypeDesc, const FPlatformTypeLayoutParameters& LayoutParams, FSHA1& Hasher)
{
	const uint32 ValueAlignment = GetTargetAlignment(ValueTypeDesc, LayoutParams);
	uint32 Offset = AppendHash(KeyTypeDesc, LayoutParams, Hasher);
	Offset = Align(Offset, FMath::Min(ValueAlignment, LayoutParams.MaxFieldAlignment));
	Offset += AppendHash(ValueTypeDesc, LayoutParams, Hasher);
	return Offset;
}

uint32 Freeze::HashLayout(const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams, FSHAHash& OutHash)
{
	FSHA1 Hasher;
	const uint32 Size = TypeDesc.AppendHashFunc(TypeDesc, LayoutParams, Hasher);
	Hasher.Final();
	FSHAHash Hash;
	Hasher.GetHash(OutHash.Hash);
	return Size;
}

FSHAHash Freeze::HashLayout(const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams)
{
	FSHAHash Result;
	HashLayout(TypeDesc, LayoutParams, Result);
	return Result;
}

uint32 Freeze::HashLayouts(const TArray<const FTypeLayoutDesc*>& TypeLayouts, const FPlatformTypeLayoutParameters& LayoutParams, FSHAHash& OutHash)
{
	FSHA1 Hasher;
	uint32 Size = 0u;
	for (const FTypeLayoutDesc* TypeDesc : TypeLayouts)
	{
		Size += TypeDesc->AppendHashFunc(*TypeDesc, LayoutParams, Hasher);
	}
	Hasher.Final();
	FSHAHash Hash;
	Hasher.GetHash(OutHash.Hash);
	return Size;
}

uint32 Freeze::DefaultUnfrozenCopy(const FMemoryUnfreezeContent& Context, const void* FrozenObject, const FTypeLayoutDesc& TypeDesc, void* OutDst)
{
	uint32 FrozenOffset = 0u;
	uint32 FieldIndex = 0u;
	uint32 NumEmptyBases = 0u;
	uint32 PrevOffset = ~0u;

	if (ETypeLayoutInterface::HasVTable(TypeDesc.Interface) && TypeDesc.NumVirtualBases == 0u)
	{
		// Copy v-table, TODO
		FMemory::Memcpy(OutDst, FrozenObject, sizeof(void*));
		FrozenOffset += Context.FrozenLayoutParameters.GetRawPointerSize();
	}

	const FFieldLayoutDesc* FieldDesc = TypeDesc.Fields;
	while (FieldDesc)
	{
		const FTypeLayoutDesc& FieldType = *FieldDesc->Type;
		const uint32 FieldTypeSize = FieldType.Size;
		uint8* FieldDst = (uint8*)OutDst + FieldDesc->Offset;

		if (IncludeField(FieldDesc, Context.FrozenLayoutParameters))
		{
			if (FieldDesc->BitFieldSize == 0u || FieldDesc->Offset != PrevOffset)
			{
				const bool bIsBase = (FieldIndex < TypeDesc.NumBases);
				const uint32 FieldTypeAlignment = GetTargetAlignment(FieldType, Context.FrozenLayoutParameters);
				const uint32 FieldAlignment = FMath::Min(FieldTypeAlignment, Context.FrozenLayoutParameters.MaxFieldAlignment);

				FrozenOffset = Align(FrozenOffset, FieldAlignment);

				FTypeLayoutDesc::FUnfrozenCopyFunc* Func = FieldType.UnfrozenCopyFunc;
				const uint8* FrozenFieldObject = (uint8*)FrozenObject + FrozenOffset;

				uint32 PaddedFrozenFieldSize = Func(Context, FrozenFieldObject, FieldType, FieldDst);
				if (PaddedFrozenFieldSize == 0u && bIsBase)
				{
					if (NumEmptyBases > 0u)
					{
						PaddedFrozenFieldSize = 1u;
					}
					else
					{
						// Empty bases are allowed to have Offset 0, since they logically overlap
						++NumEmptyBases;
					}
				}
				else
				{
					if (PaddedFrozenFieldSize == 0u)
					{
						PaddedFrozenFieldSize = 1u;
					}
				}

				if (PaddedFrozenFieldSize > 0u)
				{
					if (!bIsBase || Context.FrozenLayoutParameters.HasAlignBases())
					{
						PaddedFrozenFieldSize = Align(PaddedFrozenFieldSize, FieldTypeAlignment);
					}
				}

				FieldDst += FieldTypeSize;
				FrozenFieldObject += PaddedFrozenFieldSize;
				FrozenOffset += PaddedFrozenFieldSize;

				for (uint32 i = 1u; i < FieldDesc->NumArray; ++i)
				{
					Func(Context, FrozenFieldObject, FieldType, FieldDst);
					FieldDst += FieldTypeSize;
					FrozenFieldObject += PaddedFrozenFieldSize;
					FrozenOffset += PaddedFrozenFieldSize;
				}
			}
		}
		else
		{
			const void* DefaultObject = FieldType.GetDefaultObjectFunc ? FieldType.GetDefaultObjectFunc() : nullptr;
			if (DefaultObject)
			{
				FMemory::Memcpy(FieldDst, DefaultObject, FieldTypeSize);
			}
			else
			{
				// If no 'GetDefaultObjectFunc', assume default object is simply zero-initialized
				FMemory::Memzero(FieldDst, FieldTypeSize);
			}
		}

		++FieldIndex;
		PrevOffset = FieldDesc->Offset;
		FieldDesc = FieldDesc->Next;
	}

	return FrozenOffset;
}

void Freeze::IntrinsicWriteMemoryImage(FMemoryImageWriter& Writer, const void* Object, uint32 Size)
{
	Writer.WriteBytes(Object, Size);
}

void Freeze::IntrinsicWriteMemoryImage(FMemoryImageWriter& Writer, void*, const FTypeLayoutDesc&)
{
	Writer.WriteRawPointerSizedBytes(0u);
}

#if WITH_EDITORONLY_DATA
static void AppendNumber(ANSICHAR* Dst, int32 Num)
{
	const ANSICHAR* DigitToChar = "9876543210123456789";
	constexpr int32 ZeroDigitIndex = 9;
	bool bIsNumberNegative = Num < 0;
	const int32 TempBufferSize = 16; // 16 is big enough
	ANSICHAR TempNum[TempBufferSize];
	int32 TempAt = TempBufferSize; // fill the temp string from the top down.

	// Convert to string assuming base ten.
	do
	{
		TempNum[--TempAt] = DigitToChar[ZeroDigitIndex + (Num % 10)];
		Num /= 10;
	} while (Num);

	if (bIsNumberNegative)
	{
		TempNum[--TempAt] = TEXT('-');
	}

	const ANSICHAR* CharPtr = TempNum + TempAt;
	const int32 NumChars = TempBufferSize - TempAt;
	FMemory::Memcpy(Dst, CharPtr, NumChars);
	Dst[NumChars] = 0;
}

class FHashedNameRegistry
{
public:
	static FHashedNameRegistry& Get()
	{
		static FHashedNameRegistry Instance;
		return Instance;
	}

	FHashedNameRegistry()
	{
		const char NoneString[] = "None";
		EmptyString = RegisterString(FName(), NoneString, sizeof(NoneString), "", 0u);
	}

	const char* FindString(uint64 InHash)
	{
		FReadScopeLock ReadLock(Lock);
		const FStringEntry* Entry = Entries.Find(InHash);
		if (Entry)
		{
			return Entry->String;
		}
		return EmptyString;
	}

	const char* RegisterString(const FName& InName, const char* InString, int32 InLength, const char* InHashedString, uint64 InHash)
	{
		FStringEntry Result;
		{
			FReadScopeLock ReadLock(Lock);
			const FStringEntry* Entry = Entries.Find(InHash);
			if (Entry)
			{
				Result = *Entry;
			}
		}

		if (!Result.String)
		{
			FWriteScopeLock WriteLock(Lock);
			FStringEntry* Entry = Entries.Find(InHash);
			if (Entry)
			{
				Result = *Entry;
			}
			else
			{
				char* InternedString = nullptr;
				const int32 Number = InName.GetNumber();
				if (Number == NAME_NO_NUMBER_INTERNAL)
				{
					InternedString = (char*)MemStack.Alloc(InLength + 1, 4); // need to align debug string, to ensure we have free bits for TMemoryImagePtr
					FMemory::Memcpy(InternedString, InString, InLength);
					InternedString[InLength] = 0;
				}
				else
				{
					InternedString = (char*)MemStack.Alloc(InLength + 17, 4);
					FMemory::Memcpy(InternedString, InString, InLength);
					InternedString[InLength] = '_';
					AppendNumber(&InternedString[InLength + 1], NAME_INTERNAL_TO_EXTERNAL(Number));
				}

				TCHAR NameString[NAME_SIZE + 32];
				InName.ToString(NameString);
				checkSlow(FCString::Stricmp(NameString, UTF8_TO_TCHAR(InternedString)) == 0);

				UE_LOGF(LogMemoryImage, Verbose, "FHashedName: \"%ls\", \"%ls\", %016llX", NameString, UTF8_TO_TCHAR(InHashedString), InHash);

				Entry = &Entries.Add(InHash);
				Entry->String = InternedString;
				Entry->Name = InName;
				Result = *Entry;
			}
		}

		ensure(InName == Result.Name);
		return Result.String;
	}

	const char* RegisterStringUTF8(const char* InStringUTF8, int32 InLength, uint64 InHash)
	{
		const char* Result = nullptr;
		{
			FReadScopeLock ReadLock(Lock);
			const FStringEntry* Entry = Entries.Find(InHash);
			if (Entry)
			{
				Result = Entry->String;
			}
		}

		if (!Result)
		{
			FWriteScopeLock WriteLock(Lock);
			FStringEntry* Entry = Entries.Find(InHash);
			if (Entry)
			{
				Result = Entry->String;
			}
			else
			{
				char* InternedString = (char*)MemStack.Alloc(InLength + 1, 4);
				FMemory::Memcpy(InternedString, InStringUTF8, InLength);
				InternedString[InLength] = 0;

				const FUTF8ToTCHAR NameTCHAR(InStringUTF8);

				Entry = &Entries.Add(InHash);
				Entry->String = InternedString;
				Entry->Name = NameTCHAR.Get();
				Result = InternedString;
			}
		}

		return Result;
	}

	struct FStringEntry
	{
		const char* String = nullptr;
		FName Name;
	};

	FRWLock Lock;
	const char* EmptyString;
	TMap<uint64, FStringEntry> Entries;
	FMemStackBase MemStack;
};

void Freeze::IntrinsicWriteMemoryImage(FMemoryImageWriter& Writer, const FHashedNameDebugString& Object, const FTypeLayoutDesc&)
{
	const char* Data = Object.String.Get();
	FMemoryImageWriter StringWriter = Writer.WritePointer(StaticGetTypeLayoutDesc<char>());
	if (Data)
	{
		const int32 Length = FCStringAnsi::Strlen(Data);
		StringWriter.WriteBytes(Data, Length + 1); // include null-term
	}
	else
	{
		StringWriter.WriteBytes((uint8)0u);
	}
}

uint32 Freeze::IntrinsicUnfrozenCopy(const FMemoryUnfreezeContent& Context, const FHashedNameDebugString& Object, void* OutDst)
{
	const FName Name(Object.String.Get());
	const FHashedName HashedName(Name);
	FHashedNameDebugString* Result = new(OutDst) FHashedNameDebugString(HashedName.GetDebugString());
	return sizeof(Object);
}

#endif // WITH_EDITORONLY_DATA

FHashedName::FHashedName(const TCHAR* InString) : FHashedName(FName(InString)) {}
FHashedName::FHashedName(const FString& InString) : FHashedName(FName(InString.Len(), *InString)) {}

FHashedName::FHashedName(uint64 InHash) : Hash(InHash)
{
#if WITH_EDITORONLY_DATA
	DebugString.String = FHashedNameRegistry::Get().FindString(Hash);
#endif
}

FHashedName::FHashedName(const FName& InName)
{
	union FNameBuffer
	{
		TCHAR Wide[NAME_SIZE];
		ANSICHAR Ansi[NAME_SIZE];
	};

	if (!InName.IsNone())
	{
		const FNameEntry* Entry = InName.GetComparisonNameEntry();
		const int32 NameLength = Entry->GetNameLength();
		const int32 InternalNumber = InName.GetNumber();
		FNameBuffer NameBuffer;
		FNameBuffer UpperNameBuffer;
		if (Entry->IsWide())
		{
			// Name contains non-ansi characters, processing using TCHAR, converted to UTF8
			Entry->GetName(NameBuffer.Wide);
			for (int32 i = 0; i < NameLength; ++i)
			{
				UpperNameBuffer.Wide[i] = FChar::ToUpper(NameBuffer.Wide[i]);
			}
			UpperNameBuffer.Wide[NameLength] = TEXT('\0');
			const FTCHARToUTF8 UpperNameUTF8(UpperNameBuffer.Wide);
			Hash = CityHash64WithSeed((const char*)UpperNameUTF8.Get(), UpperNameUTF8.Length(), InternalNumber);
#if WITH_EDITORONLY_DATA
			{
				const FTCHARToUTF8 NameUTF8(NameBuffer.Wide);
				DebugString.String = FHashedNameRegistry::Get().RegisterString(InName, (char*)NameUTF8.Get(), NameUTF8.Length(), (char*)UpperNameUTF8.Get(), Hash);
			}
#endif
		}
		else
		{
			// Name is purely ascii, so avoid translating to TCHAR and UTF8, and just process directly as ascii (which is a subset of UTF8)
			Entry->GetAnsiName(NameBuffer.Ansi);
			for (int32 i = 0; i < NameLength; ++i)
			{
				UpperNameBuffer.Ansi[i] = FCharAnsi::ToUpper(NameBuffer.Ansi[i]);
			}
			UpperNameBuffer.Ansi[NameLength] = 0;
			Hash = CityHash64WithSeed(UpperNameBuffer.Ansi, NameLength, InternalNumber);
#if WITH_EDITORONLY_DATA
			DebugString.String = FHashedNameRegistry::Get().RegisterString(InName, (char*)NameBuffer.Ansi, NameLength, (char*)UpperNameBuffer.Ansi, Hash);
#endif
		}
	}
	else
	{
		Hash = 0u;
#if WITH_EDITORONLY_DATA
		DebugString.String = FHashedNameRegistry::Get().EmptyString;
#endif
	}
}

FHashedName::FHashedName(const FHashedName& InName) : Hash(InName.Hash)
{
#if WITH_EDITORONLY_DATA
	if (InName.DebugString.String.IsFrozen())
	{
		// If the source debug string is frozen, we need to make ourselved an interned unfrozen copy
		// It's not safe to point directly to the frozen source, as there's no guarantee of its lifetime
		const char* StringUTF8 = InName.DebugString.String.Get();
		DebugString.String = FHashedNameRegistry::Get().RegisterStringUTF8(StringUTF8, FCStringAnsi::Strlen(StringUTF8), InName.Hash);
	}
	else
	{
		// If the source isn't frozen, it must already be interned, safe to copy the pointer directly
		DebugString.String = InName.DebugString.String;
	}
#endif
}

FHashedName& FHashedName::operator=(const FHashedName& InName)
{
	FHashedName Temp = InName;
	Swap(Temp, *this);
	return *this;
}

#if WITH_EDITOR
void FHashedName::Save(FCbWriter& Writer) const
{
	Writer.BeginArray();
	Writer << GetHash();
	FUtf8StringView DebugName;
	DebugName = GetDebugString().String.Get();
	Writer << DebugName;
	Writer.EndArray();
}

bool FHashedName::TryLoad(FCbFieldView Field)
{
	*this = FHashedName();
	FCbFieldViewIterator ElementField(Field.CreateViewIterator());
	uint64 LocalHash = ElementField.AsUInt64();
	if ((ElementField++).HasError())
	{
		return false;
	}
	FUtf8StringView DebugName = ElementField.AsString();
	if ((ElementField++).HasError())
	{
		return false;
	}

	if (!DebugName.IsEmpty())
	{
		*this = FHashedName(*WriteToString<256>(DebugName));
		if (GetHash() == LocalHash)
		{
			return true;
		}
	}
	*this = FHashedName(LocalHash);
	return true;
}

bool LoadFromCompactBinary(FCbFieldView Field, FHashedName& OutValue)
{
	return OutValue.TryLoad(Field);
}
#endif // WITH_EDITOR

void FPointerTableBase::SaveToArchive(FArchive& Ar, const FPlatformTypeLayoutParameters& LayoutParams, const void* FrozenObject) const
{
	int32 NumDependencies = 0;
#if UE_MEMORYIMAGE_TRACK_TYPE_DEPENDENCIES
	NumDependencies = TypeDependencies.Num();
	Ar << NumDependencies;
	for (int32 i = 0; i < TypeDependencies.Num(); ++i)
	{
		const FTypeLayoutDesc* DependencyTypeDesc = TypeDependencies[i];
		FName Name = DependencyTypeDesc->Name;
		FSHAHash LayoutHash;
		uint32 LayoutSize = Freeze::HashLayout(*DependencyTypeDesc, LayoutParams, LayoutHash);
		Ar << Name;
		Ar << LayoutSize;
		Ar << LayoutHash;
	}
#else
	Ar << NumDependencies;
#endif
}

bool FPointerTableBase::LoadFromArchive(FArchive& Ar, const FPlatformTypeLayoutParameters& LayoutParams, void* FrozenObject)
{
	int32 NumDependencies = 0;
	Ar << NumDependencies;
#if UE_MEMORYIMAGE_TRACK_TYPE_DEPENDENCIES
	TypeDependencies.Empty(NumDependencies);
#endif

	// Waste a bit of time even in shipping builds skipping over this stuff
	// Could add a cook-time option to exclude dependencies completely
	bool bValid = true;
	for (int32 i = 0u; i < NumDependencies; ++i)
	{
		FName Name;
		uint32 SavedLayoutSize = 0u;
		FSHAHash SavedLayoutHash;
		Ar << Name;
		Ar << SavedLayoutSize;
		Ar << SavedLayoutHash;

#if UE_MEMORYIMAGE_TRACK_TYPE_DEPENDENCIES
		const FTypeLayoutDesc* DependencyType = FTypeLayoutDesc::Find(FHashedName(Name).GetHash());
		TypeDependencies.Add(DependencyType);
		if (DependencyType)
		{
			FSHAHash CheckLayoutHash;
			const uint32 CheckLayoutSize = Freeze::HashLayout(*DependencyType, LayoutParams, CheckLayoutHash);
			if (CheckLayoutSize != SavedLayoutSize)
			{
				UE_LOGF(LogMemoryImage, Error, "Mismatch size for type %ls, compiled size is %d, loaded size is %d", *Name.ToString(), CheckLayoutSize, SavedLayoutSize);
				bValid = false;
			}
			else if (CheckLayoutHash != SavedLayoutHash)
			{
				UE_LOGF(LogMemoryImage, Error, "Mismatch hash for type %ls", *Name.ToString());
				bValid = false;
			}
		}
		else
		{
			// When the content was cooked with a different build configuration than the editor, some type layouts are unknown to the editor build.
			// This condition can be considered a benign warning, so we only emit it as a Verbose log entry.
			UE_LOGF(LogMemoryImage, Verbose, "Unknown TypeLayout %ls", *Name.ToString());
		}
#endif // UE_MEMORYIMAGE_TRACK_TYPE_DEPENDENCIES
	}

	return bValid;
}

#if UE_MEMORYIMAGE_TRACK_TYPE_DEPENDENCIES
int32 FPointerTableBase::AddTypeDependency(const FTypeLayoutDesc& TypeDesc)
{
	if (TypeDesc.NameHash != 0u)
	{
		for (int32 Index = 0; Index < TypeDependencies.Num(); ++Index)
		{
			if (TypeDependencies[Index] != nullptr && TypeDependencies[Index]->NameHash == TypeDesc.NameHash)
			{
				return Index;
			}
		}

		return TypeDependencies.Add(&TypeDesc);
	}
	return INDEX_NONE;
}
#endif // UE_MEMORYIMAGE_TRACK_TYPE_DEPENDENCIES

FMemoryImageObject FreezeMemoryImageObject(const void* Object, const FTypeLayoutDesc& TypeDesc, FPointerTableBase* PointerTable)
{
	FMemoryImage MemoryImage;
	MemoryImage.TargetLayoutParameters.InitializeForCurrent();
	MemoryImage.PointerTable = PointerTable;
	FMemoryImageWriter Writer(MemoryImage);

	Writer.WriteRootObject(Object, TypeDesc);

	FMemoryImageResult MemoryImageResult;
	MemoryImage.Flatten(MemoryImageResult, true);

	const uint32 FrozenSize = MemoryImageResult.Bytes.Num();
	check(FrozenSize > 0u);
	void* FrozenObject = FMemory::Malloc(FrozenSize);
	FMemory::Memcpy(FrozenObject, MemoryImageResult.Bytes.GetData(), FrozenSize);
	if (UNLIKELY(!MemoryImageResult.ApplyPatches(FrozenObject, FrozenSize)))
	{
		return FMemoryImageObject();
	}

	return FMemoryImageObject(TypeDesc, FrozenObject, FrozenSize);
}

void* UnfreezeMemoryImageObject(const void* FrozenObject, const FTypeLayoutDesc& TypeDesc, const FPointerTableBase* PointerTable)
{
	void* UnfrozenMemory = ::operator new(TypeDesc.Size);
	FMemoryUnfreezeContent Context(PointerTable);
	Context.UnfreezeObject(FrozenObject, TypeDesc, UnfrozenMemory);
	return UnfrozenMemory;
}

static void CountNumNames(const TArray<FMemoryImageNamePointer>& Names, TArray<uint32>& OutNameCounts)
{
	FName CurrentName;
	uint32 CurrentNumPatches = 0u;
	for (const FMemoryImageNamePointer& Patch : Names)
	{
		if (Patch.Name.Compare(CurrentName) != 0)
		{
			if (CurrentNumPatches > 0u)
			{
				OutNameCounts.Add(CurrentNumPatches);
			}
			CurrentName = Patch.Name;
			CurrentNumPatches = 0u;
		}
		++CurrentNumPatches;
	}
	if (CurrentNumPatches > 0u)
	{
		OutNameCounts.Add(CurrentNumPatches);
	}
}

static void SerializeNames(const TArray<FMemoryImageNamePointer>& Names, const TArray<uint32>& NameCounts, FArchive& Ar)
{
	int32 NameIndex = 0;
	for (uint32 Num : NameCounts)
	{
		FName Name = Names[NameIndex].Name;
		Ar << Name;
		Ar << Num;

		for (uint32 i = 0; i < Num; ++i)
		{
			const FMemoryImageNamePointer& Patch = Names[NameIndex++];
			check(Patch.Name.Compare(Name) == 0);

			uint32 Offset = Patch.Offset;
			Ar << Offset;
		}
	}
	check(NameIndex == Names.Num());
}

void FMemoryImageResult::SaveToArchive(FArchive& Ar) const
{
	uint32 FrozenSize = Bytes.Num();
	uint8* FrozenObject = const_cast<uint8*>(Bytes.GetData());

	Ar << const_cast<FMemoryImageResult*>(this)->TargetLayoutParameters;

	Ar << FrozenSize;
	int64 FrozenObjectOffset = Ar.Tell();
	Ar.Serialize(FrozenObject, FrozenSize);

	PointerTable->SaveToArchive(Ar, TargetLayoutParameters, FrozenObject);

	TArray<uint32> VTableCounts;
	{
		uint64 CurrentTypeNameHash = 0u;
		uint32 CurrentNumPatches = 0u;
		for (const FMemoryImageVTablePointer& Patch : VTables)
		{
			if (Patch.TypeNameHash != CurrentTypeNameHash)
			{
				if (CurrentNumPatches > 0u)
				{
					VTableCounts.Add(CurrentNumPatches);
				}
				CurrentTypeNameHash = Patch.TypeNameHash;
				CurrentNumPatches = 0u;
			}
			++CurrentNumPatches;
		}
		if (CurrentNumPatches > 0u)
		{
			VTableCounts.Add(CurrentNumPatches);
		}
	}


	TArray<uint32> ScriptNameCounts;
	TArray<uint32> MemoryImageNameCounts;
	CountNumNames(ScriptNames, ScriptNameCounts);
	CountNumNames(MemoryImageNames, MemoryImageNameCounts);

	uint32 NumVTables = VTableCounts.Num();
	uint32 NumScriptNames = ScriptNameCounts.Num();
	uint32 NumMemoryImageNames = MemoryImageNameCounts.Num();
	Ar << NumVTables;
	Ar << NumScriptNames;
	Ar << NumMemoryImageNames;

	{
		int32 VTableIndex = 0;
		for (uint32 Num : VTableCounts)
		{
			uint64 TypeNameHash = VTables[VTableIndex].TypeNameHash;
			Ar << TypeNameHash;
			Ar << Num;

			for (uint32 i = 0; i < Num; ++i)
			{
				const FMemoryImageVTablePointer& Patch = VTables[VTableIndex++];
				check(Patch.TypeNameHash == TypeNameHash);

				uint32 VTableOffset = Patch.VTableOffset;
				uint32 Offset = Patch.Offset;
				Ar << VTableOffset;
				Ar << Offset;
			}
		}
		check(VTableIndex == VTables.Num());
	}
	
	SerializeNames(ScriptNames, ScriptNameCounts, Ar);
	SerializeNames(MemoryImageNames, MemoryImageNameCounts, Ar);

	int64 EndOffset = Ar.Tell();

	// Replace patchable values with zeroes for determinism. 
	// We can't do this when we flatten an object because we may still want to read that object on the current platform. 
	for (const FMemoryImageVTablePointer& Patch : VTables)
	{
		Ar.Seek(FrozenObjectOffset + Patch.Offset);
		if (TargetLayoutParameters.Is32Bit())
		{
			int32 Zero = 0;
			Ar.Serialize(&Zero, sizeof(Zero));
		}
		else
		{
			int64 Zero = 0;
			Ar.Serialize(&Zero, sizeof(Zero));
		}
	}

	for (const FMemoryImageNamePointer& Patch : ScriptNames)
	{
		Ar.Seek(FrozenObjectOffset + Patch.Offset);
		FScriptName NullName;
		Ar.Serialize(&NullName, sizeof(FScriptName));
	}

	for (const FMemoryImageNamePointer& Patch : MemoryImageNames)
	{
		Ar.Seek(FrozenObjectOffset + Patch.Offset);
		int32 Zeroes[3] = {0, 0, 0};
		if (TargetLayoutParameters.WithCasePreservingFName())
		{
			Ar.Serialize(Zeroes, sizeof(int32) * 3);
		}
		else
		{
			Ar.Serialize(Zeroes, sizeof(int32) * 2);
		}
	}

	Ar.Seek(EndOffset);
}

static inline bool ApplyVTablePatch(void* FrozenObject, uint64 FrozenObjectSize, const FTypeLayoutDesc & DerivedType, uint32 VTableOffset, uint32 Offset)
{
	if (UNLIKELY(Offset > (FrozenObjectSize - sizeof(void*)) || VTableOffset > (DerivedType.Size - sizeof(void*))))
	{
		return false;
	}

	const void** VTableSrc = (void const**)((uint8*)DerivedType.GetDefaultObjectFunc() + VTableOffset);
	void const** VTableDst = (void const**)((uint8*)FrozenObject + Offset);
	*VTableDst = *VTableSrc;
	return true;
}

static inline bool ApplyScriptNamePatch(void* FrozenObject, uint64 FrozenObjectSize, const FScriptName& Name, uint32 Offset)
{
	if (UNLIKELY(Offset > FrozenObjectSize - sizeof(FScriptName)))
	{
		return false;
	}

	void* NameDst = (uint8*)FrozenObject + Offset;
	new(NameDst) FScriptName(Name);
	return true;
}

static inline bool ApplyMemoryImageNamePatch(void* FrozenObject, uint64 FrozenObjectSize, const FMemoryImageName& Name, uint32 Offset, const FPlatformTypeLayoutParameters& LayoutParameters)
{
	if (UNLIKELY(Offset > FrozenObjectSize - sizeof(FMemoryImageName)))
	{
		return false;
	}

	void* NameDst = (uint8*)FrozenObject + Offset;
	if (LayoutParameters.IsCurrentPlatform())
	{
		new(NameDst) FMemoryImageName(Name);
	}
	else
	{
		Freeze::ApplyMemoryImageNamePatch(NameDst, Name, LayoutParameters);
	}

	return true;
}

bool FMemoryImageResult::ApplyPatches(void* FrozenObject, uint64 FrozenObjectSize) const
{
	for (const FMemoryImageVTablePointer& Patch : VTables)
	{
		const FTypeLayoutDesc* DerivedType = FTypeLayoutDesc::Find(Patch.TypeNameHash);
		check(DerivedType);
		if (!ApplyVTablePatch(FrozenObject, FrozenObjectSize, *DerivedType, Patch.VTableOffset, Patch.Offset))
		{
			return false;
		}
	}

	for (const FMemoryImageNamePointer& Patch : ScriptNames)
	{
		if (!ApplyScriptNamePatch(FrozenObject, FrozenObjectSize, NameToScriptName(Patch.Name), Patch.Offset))
		{
			return false;
		}
	}

	for (const FMemoryImageNamePointer& Patch : MemoryImageNames)
	{
		if (!ApplyMemoryImageNamePatch(FrozenObject, FrozenObjectSize, FMemoryImageName(Patch.Name), Patch.Offset, TargetLayoutParameters))
		{
			return false;
		}
	}

	return true;
}

FMemoryImageObject FMemoryImageResult::LoadFromArchive(FArchive& Ar, const FTypeLayoutDesc& TypeDesc, FPointerTableBase* PointerTable, FPlatformTypeLayoutParameters& LayoutParameters)
{
	SCOPED_LOADTIMER(FMemoryImageResult_LoadFromArchive);

	Ar << LayoutParameters;

	uint32 FrozenSize = 0u;
	Ar << FrozenSize;
	void* FrozenObject = FMemory::Malloc(FrozenSize);
	Ar.Serialize(FrozenObject, FrozenSize);

	bool bFrozenObjectValid = PointerTable->LoadFromArchive(Ar, LayoutParameters, FrozenObject);

	uint32 NumVTables = 0u;
	uint32 NumScriptNames = 0u;
	uint32 NumMemoryImageNames = 0u;
	Ar << NumVTables;
	Ar << NumScriptNames;
	Ar << NumMemoryImageNames;

	// if image read is invalid (or if we find out when patching that offsets are invalid),
	// we will still keep reading the image data from the archive as we cannot rewind

	// clamp to reasonable numbers
	const uint32 kReasonableMemoryImageElementCount = 16u * 1024u * 1024u;

	ensure(NumVTables < kReasonableMemoryImageElementCount);
	NumVTables = FMath::Min(NumVTables, kReasonableMemoryImageElementCount);

	for (uint32 i = 0u; i < NumVTables; ++i)
	{
		uint64 TypeNameHash = 0u;
		uint32 NumPatches = 0u;
		Ar << TypeNameHash;
		Ar << NumPatches;

		const FTypeLayoutDesc* DerivedType = bFrozenObjectValid ? FTypeLayoutDesc::Find(TypeNameHash) : nullptr;
		check(DerivedType || !bFrozenObjectValid);

		ensure(NumPatches < kReasonableMemoryImageElementCount);
		NumPatches = FMath::Min(NumPatches, kReasonableMemoryImageElementCount);

		for(uint32 PatchIndex = 0u; PatchIndex < NumPatches; ++PatchIndex)
		{
			uint32 VTableOffset = 0u;
			uint32 Offset = 0u;
			Ar << VTableOffset;
			Ar << Offset;
			if (bFrozenObjectValid && !ApplyVTablePatch(FrozenObject, FrozenSize, *DerivedType, VTableOffset, Offset))
			{
				bFrozenObjectValid = false;
			}
		}
	}

	ensure(NumScriptNames < kReasonableMemoryImageElementCount);
	NumScriptNames = FMath::Min(NumScriptNames, kReasonableMemoryImageElementCount);

	for (uint32 i = 0u; i < NumScriptNames; ++i)
	{
		FName Name;
		uint32 NumPatches = 0u;
		Ar << Name;
		Ar << NumPatches;

		ensure(NumPatches < kReasonableMemoryImageElementCount);
		NumPatches = FMath::Min(NumPatches, kReasonableMemoryImageElementCount);

		for (uint32 PatchIndex = 0u; PatchIndex < NumPatches; ++PatchIndex)
		{
			uint32 Offset = 0u;
			Ar << Offset;
			if (bFrozenObjectValid && !ApplyScriptNamePatch(FrozenObject, FrozenSize, NameToScriptName(Name), Offset))
			{
				bFrozenObjectValid = false;
			}
		}
	}

	ensure(NumMemoryImageNames < kReasonableMemoryImageElementCount);
	NumMemoryImageNames = FMath::Min(NumMemoryImageNames, kReasonableMemoryImageElementCount);

	for (uint32 i = 0u; i < NumMemoryImageNames; ++i)
	{
		FName Name;
		uint32 NumPatches = 0u;
		Ar << Name;
		Ar << NumPatches;

		ensure(NumPatches < kReasonableMemoryImageElementCount);
		NumPatches = FMath::Min(NumPatches, kReasonableMemoryImageElementCount);

		for (uint32 PatchIndex = 0u; PatchIndex < NumPatches; ++PatchIndex)
		{
			uint32 Offset = 0u;
			Ar << Offset;
			if (bFrozenObjectValid && !ApplyMemoryImageNamePatch(FrozenObject, FrozenSize, FMemoryImageName(Name), Offset, LayoutParameters))
			{
				bFrozenObjectValid = false;
			}
		}
	}

	if (!bFrozenObjectValid)
	{
		FMemory::Free(FrozenObject);
		return FMemoryImageObject();
	}
	else if (LayoutParameters.IsCurrentPlatform())
	{
		// Loading content that matches the current platform, just use frozen content directly (expected case)
		return FMemoryImageObject(TypeDesc, FrozenObject, FrozenSize);
	}
	else
	{
		// Loading content that was frozen for a different platform/configuration, need to convert
		// This path is only supported if we're tracking type dependencies, otherwise we won't have any TypeDependencies stored in the pointer table, which is required for conversion
		void* UnfrozenObject = nullptr;
#if UE_MEMORYIMAGE_TRACK_TYPE_DEPENDENCIES
		UnfrozenObject = ::operator new(TypeDesc.Size);

		FMemoryUnfreezeContent UnfreezeContext(PointerTable, LayoutParameters);
		UnfreezeContext.UnfreezeObject(FrozenObject, TypeDesc, UnfrozenObject);
#else
		FPlatformTypeLayoutParameters CurrentPlatform;
		CurrentPlatform.InitializeForCurrent();

		UE_LOGF(LogMemoryImage, Error, "Attempting to load an incompatible memory image and conversion isn't supported. Image layout: MaxFieldAlignment=%u, Flags=0x%x, CurrentPlatform: MaxFieldAlignment=%u, Flags=0x%x",
			LayoutParameters.MaxFieldAlignment, LayoutParameters.Flags, CurrentPlatform.MaxFieldAlignment, CurrentPlatform.Flags
			);
#endif // UE_MEMORYIMAGE_TRACK_TYPE_DEPENDENCIES
		FMemory::Free(FrozenObject);
		return FMemoryImageObject(TypeDesc, UnfrozenObject, 0u);
	}
}

void FPtrTableBase::SavePatchesToArchive(FArchive& Ar, uint32 PtrIndex) const
{
	if (PtrIndex < (uint32)PatchLists.Num())
	{
		const FPatchOffsetList& List = PatchLists[PtrIndex];
		int32 NumOffsets = List.NumOffsets;
		Ar << NumOffsets;
		uint32 OffsetIndex = List.FirstIndex;
		while (OffsetIndex != ~0u)
		{
			const FPatchOffset& Patch = PatchOffsets[OffsetIndex];
			uint32 Offset = Patch.Offset;
			Ar << Offset;
			OffsetIndex = Patch.NextIndex;
		}
	}
	else
	{
		int32 NumOffsets = 0;
		Ar << NumOffsets;
	}
}

void FPtrTableBase::AddPatchedPointerBase(uint32 PtrIndex, uint64 Offset)
{
	if (PtrIndex >= (uint32)PatchLists.Num())
	{
		PatchLists.SetNum(PtrIndex + 1, EAllowShrinking::No);
	}
	FPatchOffsetList& List = PatchLists[PtrIndex];

	const uint32 OffsetIndex = PatchOffsets.AddUninitialized();
	PatchOffsets[OffsetIndex].Offset = (int32)Offset;
	PatchOffsets[OffsetIndex].NextIndex = List.FirstIndex;
	List.FirstIndex = OffsetIndex;
	++List.NumOffsets;
}

FMemoryImageSection* FMemoryImageSection::WritePointer(const FTypeLayoutDesc& StaticTypeDesc, const FTypeLayoutDesc& DerivedTypeDesc, uint32* OutOffsetToBase)
{
	FSectionPointer& SectionPointer = Pointers.AddDefaulted_GetRef();

	const uint32 OffsetToBase = DerivedTypeDesc.GetOffsetToBase(StaticTypeDesc);
	const int32 TypeDependencyIndex = ParentImage->PointerTable->AddTypeDependency(DerivedTypeDesc);
	const bool bStaticTypeMatchesDerived = (StaticTypeDesc == DerivedTypeDesc);

	if (OutOffsetToBase)
	{
		*OutOffsetToBase = OffsetToBase;
	}

	checkf(OutOffsetToBase || bStaticTypeMatchesDerived, TEXT("Must consider OffsetToBase if static/derived types are different, %s/%s"), DerivedTypeDesc.Name, StaticTypeDesc.Name);
	checkf(TypeDependencyIndex != INDEX_NONE || bStaticTypeMatchesDerived,
		TEXT("Unable to store pointer to derived type %s, different from static type %s\n"
		     "Ensure UE_MEMORYIMAGE_TRACK_TYPE_DEPENDENCIES is set\n"
		     "Make sure derived type is not declared using DECLARE_INLINE_TYPE_LAYOUT()"),
		DerivedTypeDesc.Name, StaticTypeDesc.Name);

	FFrozenMemoryImagePtr FrozenPtr;
	FrozenPtr.Packed = 0; // dummy value
	// If derived type matches the static type, store INDEX_NONE to indicate the static type may be used
	FrozenPtr.SetTypeIndex(bStaticTypeMatchesDerived ? INDEX_NONE : TypeDependencyIndex);

	SectionPointer.Offset = WriteBytes(FrozenPtr);
	SectionPointer.PointerOffset = OffsetToBase;
	SectionPointer.SectionIndex = ParentImage->Sections.Num();
	return ParentImage->AllocateSection();
}

uint32 FMemoryImageSection::WriteRawPointerSizedBytes(uint64 PointerValue)
{
	if (ParentImage->TargetLayoutParameters.Is32Bit())
	{
		return WriteBytes((uint32)PointerValue);
	}
	else
	{
		return WriteBytes(PointerValue);
	}
}

uint32 FMemoryImageSection::WriteVTable(const FTypeLayoutDesc& TypeDesc, const FTypeLayoutDesc& DerivedTypeDesc)
{
	checkf(DerivedTypeDesc.NameHash != 0u, TEXT("Type %s is not registered"), DerivedTypeDesc.Name);
	checkf(DerivedTypeDesc.Interface == ETypeLayoutInterface::Virtual, TEXT("Type %s is not virtual"), DerivedTypeDesc.Name);

	FMemoryImageVTablePointer& VTablePointer = VTables.AddDefaulted_GetRef();
	VTablePointer.Offset = WriteRawPointerSizedBytes((uint64)-1); // write dummy value
	VTablePointer.TypeNameHash = DerivedTypeDesc.NameHash;
	VTablePointer.VTableOffset = DerivedTypeDesc.GetOffsetToBase(TypeDesc);
	return VTablePointer.Offset;
}

uint32 FMemoryImageSection::WriteFMemoryImageName(int32 NumBytes, const FName& Name)
{
	const uint32 Offset = WriteZeroBytes(NumBytes);
	if (!Name.IsNone())
	{
		FMemoryImageNamePointer& NamePointer = MemoryImageNames.AddDefaulted_GetRef();
		NamePointer.Name = Name;
		NamePointer.Offset = Offset;
	}
	return Offset;
}

uint32 FMemoryImageSection::WriteFScriptName(const FScriptName& Name)
{
	const uint32 Offset = WriteZeroBytes(sizeof(Name));
	if (!Name.IsNone())
	{
		FMemoryImageNamePointer& NamePointer = ScriptNames.AddDefaulted_GetRef();
		NamePointer.Name = ScriptNameToName(Name);
		NamePointer.Offset = Offset;
	}
	return Offset;
}

uint32 FMemoryImageSection::Flatten(FMemoryImageResult& OutResult) const
{
	const uint32 AlignedOffset = (OutResult.Bytes.Num() + MaxAlignment - 1) & ~(MaxAlignment - 1u);

	OutResult.Bytes.SetNumZeroed(AlignedOffset + Bytes.Num());
	FMemory::Memcpy(&OutResult.Bytes[AlignedOffset], Bytes.GetData(), Bytes.Num());

	OutResult.VTables.Reserve(OutResult.VTables.Num() + VTables.Num());
	for (const FMemoryImageVTablePointer& VTable : VTables)
	{
		FMemoryImageVTablePointer& ResultVTable = OutResult.VTables.Add_GetRef(VTable);
		ResultVTable.Offset += AlignedOffset;
	}

	OutResult.ScriptNames.Reserve(OutResult.ScriptNames.Num() + ScriptNames.Num());
	for (const FMemoryImageNamePointer& Name : ScriptNames)
	{
		FMemoryImageNamePointer& ResultName = OutResult.ScriptNames.Add_GetRef(Name);
		ResultName.Offset += AlignedOffset;
	}

	OutResult.MemoryImageNames.Reserve(OutResult.MemoryImageNames.Num() + MemoryImageNames.Num());
	for (const FMemoryImageNamePointer& Name : MemoryImageNames)
	{
		FMemoryImageNamePointer& ResultName = OutResult.MemoryImageNames.Add_GetRef(Name);
		ResultName.Offset += AlignedOffset;
	}

	return AlignedOffset;
}

static void UpdateHashNamePatches(const TArray< FMemoryImageNamePointer>& Names, FSHA1& HashState)
{
	for (const FMemoryImageNamePointer& NamePatch : Names)
	{
		const FNameEntry* NameEntry = NamePatch.Name.GetDisplayNameEntry();
		TCHAR NameBuffer[NAME_SIZE];
		NameEntry->GetName(NameBuffer);
		const int32 Number = NamePatch.Name.GetNumber();
		HashState.UpdateWithString(NameBuffer, NameEntry->GetNameLength());
		HashState.Update((uint8*)&Number, sizeof(Number));
		HashState.Update((uint8*)&NamePatch.Offset, sizeof(NamePatch.Offset));
	}
}

void FMemoryImageSection::ComputeHash()
{
	FSHA1 HashState;
	HashState.Update(Bytes.GetData(), Bytes.Num());
	HashState.Update((uint8*)Pointers.GetData(), Pointers.Num() * Pointers.GetTypeSize());
	HashState.Update((uint8*)VTables.GetData(), VTables.Num() * VTables.GetTypeSize());
	UpdateHashNamePatches(ScriptNames, HashState);
	UpdateHashNamePatches(MemoryImageNames, HashState);
	HashState.Final();
	HashState.GetHash(Hash.Hash);
}

void FMemoryImage::Flatten(FMemoryImageResult& OutResult, bool bMergeDuplicateSections)
{
	TArray<FMemoryImageSection*> UniqueSections;
	UniqueSections.Reserve(Sections.Num());

	TArray<int32> SectionIndexRemap;
	SectionIndexRemap.Init(INDEX_NONE, Sections.Num());

	if(bMergeDuplicateSections)
	{
		// Find unique sections
		// precompute hashes in parallel
		ParallelFor(Sections.Num(), [this](int32 Index) { Sections[Index]->ComputeHash(); }, EParallelForFlags::Unbalanced);

		TMap<FSHAHash, int32> HashToSectionIndex;
		for (int32 SectionIndex = 0; SectionIndex < Sections.Num(); ++SectionIndex)
		{
			FMemoryImageSection* Section = Sections[SectionIndex];
			int32* FoundIndex = HashToSectionIndex.Find(Section->Hash);
			int32 NewIndex = INDEX_NONE;
			if (!FoundIndex)
			{
				NewIndex = UniqueSections.Add(Section);
				HashToSectionIndex.Add(Section->Hash, NewIndex);
			}
			else
			{
				NewIndex = *FoundIndex;
			}
			SectionIndexRemap[SectionIndex] = NewIndex;
		}
	}
	else
	{
		for (int32 SectionIndex = 0; SectionIndex < Sections.Num(); ++SectionIndex)
		{
			UniqueSections.Add(Sections[SectionIndex]);
			SectionIndexRemap[SectionIndex] = SectionIndex;
		}
	}

	TArray<uint32> SectionOffset;
	SectionOffset.SetNum(UniqueSections.Num());

	for (int32 SectionIndex = 0; SectionIndex < UniqueSections.Num(); ++SectionIndex)
	{
		const FMemoryImageSection* Section = UniqueSections[SectionIndex];
		SectionOffset[SectionIndex] = Section->Flatten(OutResult);
	}

	for (int32 SectionIndex = 0; SectionIndex < UniqueSections.Num(); ++SectionIndex)
	{
		const FMemoryImageSection* Section = UniqueSections[SectionIndex];
		for (const FMemoryImageSection::FSectionPointer& Pointer : Section->Pointers)
		{
			const int64 OffsetToPointer = SectionOffset[SectionIndex] + Pointer.Offset;
			const int32 RemapSectionIndex = SectionIndexRemap[Pointer.SectionIndex];
			FFrozenMemoryImagePtr* FrozenPtr = (FFrozenMemoryImagePtr*)(OutResult.Bytes.GetData() + OffsetToPointer);
			check(FrozenPtr->GetOffsetFromThis() == 0 && !FrozenPtr->IsFrozen());
			const int64 OffsetFromPointer = static_cast<int64>(SectionOffset[RemapSectionIndex]) + static_cast<int64>(Pointer.PointerOffset) - OffsetToPointer;
			FrozenPtr->SetOffsetFromThis(OffsetFromPointer);
			FrozenPtr->SetIsFrozen(true);
			check(FrozenPtr->GetOffsetFromThis() == OffsetFromPointer);
		}
	}

	OutResult.PointerTable = PointerTable;
	OutResult.TargetLayoutParameters = TargetLayoutParameters;

	// Sort to group runs of the same FName/VTable
	OutResult.VTables.Sort();
	OutResult.ScriptNames.Sort();
	OutResult.MemoryImageNames.Sort();
}

FMemoryImageWriter::FMemoryImageWriter(FMemoryImage& InImage) : Section(InImage.AllocateSection()) {}
FMemoryImageWriter::FMemoryImageWriter(FMemoryImageSection* InSection) : Section(InSection) {}

FMemoryImageWriter::~FMemoryImageWriter()
{
}

FMemoryImage& FMemoryImageWriter::GetImage() const { return *Section->ParentImage; }
const FPlatformTypeLayoutParameters& FMemoryImageWriter::GetHostLayoutParams() const { return GetImage().HostLayoutParameters; }
const FPlatformTypeLayoutParameters& FMemoryImageWriter::GetTargetLayoutParams() const { return GetImage().TargetLayoutParameters; }
FPointerTableBase& FMemoryImageWriter::GetPointerTable() const { return GetImage().GetPointerTable(); }
const FPointerTableBase* FMemoryImageWriter::TryGetPrevPointerTable() const { return GetImage().PrevPointerTable; }

int32 FMemoryImageWriter::AddTypeDependency(const FTypeLayoutDesc& TypeDesc)
{
	return GetPointerTable().AddTypeDependency(TypeDesc);
}

void FMemoryImageWriter::WriteObject(const void* Object, const FTypeLayoutDesc& TypeDesc)
{
	TypeDesc.WriteFrozenMemoryImageFunc(*this, Object, TypeDesc, TypeDesc);
}

void FMemoryImageWriter::WriteRootObject(const void* Object, const FTypeLayoutDesc& TypeDesc)
{
	const int32 RootTypeIndex = AddTypeDependency(TypeDesc);
#if UE_MEMORYIMAGE_TRACK_TYPE_DEPENDENCIES
	checkf(RootTypeIndex == 0, TEXT("Can't add root object of type %s, already have a root object"), TypeDesc.Name);
#endif
	TypeDesc.WriteFrozenMemoryImageFunc(*this, Object, TypeDesc, TypeDesc);
}

void FMemoryImageWriter::WriteObjectArray(const void* Object, const FTypeLayoutDesc& TypeDesc, uint32 NumArray)
{
	FTypeLayoutDesc::FWriteFrozenMemoryImageFunc* Func = TypeDesc.WriteFrozenMemoryImageFunc;
	const uint8* CurrentElement = (uint8*)Object;

	const uint32 TargetAlignment = Freeze::GetTargetAlignment(TypeDesc, GetTargetLayoutParams());
	WriteAlignment(TargetAlignment);
	for (uint32_t i = 0u; i < NumArray; ++i)
	{
		Func(*this, CurrentElement, TypeDesc, TypeDesc);
		WriteAlignment(TargetAlignment);
		CurrentElement += TypeDesc.Size;
	}
}

uint32 FMemoryImageWriter::GetOffset() const
{
	return Section->GetOffset();
}

uint32 FMemoryImageWriter::WriteAlignment(uint32 Alignment)
{
	return Section->WriteAlignment(Alignment);
}

void FMemoryImageWriter::WritePaddingToSize(uint32 Offset)
{
	Section->WritePaddingToSize(Offset);
}

uint32 FMemoryImageWriter::WriteBytes(const void* Data, uint32 Size)
{
	return Section->WriteBytes(Data, Size);
}

uint32 FMemoryImageWriter::WriteNullPointer()
{
	return WriteBytes(FFrozenMemoryImagePtr{ 0ull });
}

FMemoryImageWriter FMemoryImageWriter::WritePointer(const FTypeLayoutDesc& StaticTypeDesc, const FTypeLayoutDesc& DerivedTypeDesc, uint32* OutOffsetToBase)
{
	return FMemoryImageWriter(Section->WritePointer(StaticTypeDesc, DerivedTypeDesc, OutOffsetToBase));
}

FMemoryImageWriter FMemoryImageWriter::WritePointer(const FTypeLayoutDesc& TypeDesc)
{
	return FMemoryImageWriter(Section->WritePointer(TypeDesc, TypeDesc, nullptr));
}

uint32 FMemoryImageWriter::WriteRawPointerSizedBytes(uint64 PointerValue)
{
	return Section->WriteRawPointerSizedBytes(PointerValue);
}

uint32 FMemoryImageWriter::WriteVTable(const FTypeLayoutDesc& TypeDesc, const FTypeLayoutDesc& DerivedTypeDesc)
{
	return Section->WriteVTable(TypeDesc, DerivedTypeDesc);
}

uint32 FMemoryImageWriter::WriteFMemoryImageName(int32 NumBytes, const FName& Name)
{
	return Section->WriteFMemoryImageName(NumBytes, Name);
}

uint32 FMemoryImageWriter::WriteFScriptName(const FScriptName& Name)
{
	return Section->WriteFScriptName(Name);
}

const FTypeLayoutDesc* FMemoryUnfreezeContent::GetDerivedTypeDesc(const FTypeLayoutDesc& StaticTypeDesc, int32 TypeIndex) const
{
	if (TypeIndex == INDEX_NONE)
	{
		// TypeIndex of INDEX_NONE means the derived type was the same as the static type
		return &StaticTypeDesc;
	}
	if (PrevPointerTable)
	{
		return PrevPointerTable->GetTypeDependency(TypeIndex);
	}
	return nullptr;
}

// Finds the length of the field name, omitting any _DEPRECATED suffix
uint8 Freeze::FindFieldNameLength(const TCHAR* Name)
{
	uint8 Result = 0;

	const TCHAR* NameIter = Name;
	for (;;)
	{
		TCHAR Ch = *NameIter++;

		if (!Ch)
		{
			return Result;
		}

		if (Ch == TEXT('_'))
		{
			static const TCHAR* DeprecatedName = TEXT("DEPRECATED");

			const TCHAR* DepIter = DeprecatedName;
			for (;;)
			{
				Ch = *NameIter;
				if (Ch != *DepIter)
				{
					break;
				}

				if (!Ch)
				{
					return Result;
				}

				++NameIter;
				++DepIter;
			}

			Result += (uint8)(DepIter - DeprecatedName);
		}

		++Result;
	}
}
