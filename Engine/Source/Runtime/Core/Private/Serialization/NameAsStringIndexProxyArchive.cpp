// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/NameAsStringIndexProxyArchive.h"

#include "Containers/Array.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Serialization/Archive.h"
#include "Serialization/VarInt.h"
#include "UObject/NameTypes.h"

FArchive& FNameAsStringIndexProxyArchive::operator<<(class FName& N)
{
	if (IsLoading())
	{
		uint64 Index64 = ReadVarUIntFromArchive(InnerArchive);

		// if this is 0, then it was saved as string. If not zero, then it refers to the number of bytes in the index
		if (Index64 == 0)
		{
			FString LoadedString;
			InnerArchive << LoadedString;
			N = FName(LoadedString);

			NamesLoaded.Add(N);
		}
		else 
		{
			int32 Index = static_cast<int32>(Index64 - 1);
			if (Index >= 0 && Index < NamesLoaded.Num())
			{
				N = NamesLoaded[Index];
			}
			else
			{
				SetError();
			}
		}
	}
	else
	{
		// We rely on elements' indices in TSet being in the insertion order, which they are now and should remain so in the future.
		FSetElementId Id = NamesSeenOnSave.FindId(N);
		if (Id.IsValidId())
		{
			int32 Index = Id.AsInteger();
			WriteVarUIntToArchive(InnerArchive, uint64(Index) + 1);
		}
		else
		{
			FString SavedString(N.ToString());
			WriteVarUIntToArchive(InnerArchive, 0ULL);
			InnerArchive << SavedString;

			NamesSeenOnSave.Add(N);
		}
	}
	return *this;
}
