// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/ShaderKeyGenerator.h"

#include "Hash/Blake3.h"
#include "Misc/Guid.h"
#include "Misc/SecureHash.h"
#include "Hash/ShaderHash.h"

void FShaderKeyGenerator::Append(const FBlake3Hash& Value)
{
	switch (OutputType)
	{
	case EOutputType::Text:
	{
		CA_SUPPRESS(6260) /* warning C6260: sizeof * sizeof is usually wrong. */
		constexpr int32 StringSize = sizeof(TCHAR) * sizeof(FBlake3Hash::ByteArray) * 2; // -V531
		ResultString->Append(WriteToString<StringSize>(Value));
		return;
	}
	case EOutputType::Binary:
		ResultFunc(&Value, sizeof(Value));
		return;
	default:
		checkNoEntry();
	}
}

void FShaderKeyGenerator::Append(const FGuid& Value)
{
	switch (OutputType)
	{
	case EOutputType::Text:
		Value.AppendString(*ResultString);
		return;
	case EOutputType::Binary:
		ResultFunc(&Value, sizeof(Value));
		return;
	default:
		checkNoEntry();
	}
}

void FShaderKeyGenerator::Append(const FSHAHash& Value)
{
	switch (OutputType)
	{
	case EOutputType::Text:
		Value.AppendString(*ResultString);
		return;
	case EOutputType::Binary:
		ResultFunc(&Value.Hash, sizeof(Value.Hash));
		return;
	default:
		checkNoEntry();
	}
}

void FShaderKeyGenerator::Append(const FShaderHash& Value)
{
	switch (OutputType)
	{
	case EOutputType::Text:
		Value.AppendString(*ResultString);
		return;
	case EOutputType::Binary:
		ResultFunc(&Value.Hash, sizeof(Value.Hash));
		return;
	default:
		checkNoEntry();
	}
}
