// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/StructuredArchive.h"
#include "Serialization/StructuredArchiveContainer.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"

#if WITH_TEXT_ARCHIVE_SUPPORT

//////////// FStructuredArchive ////////////

FStructuredArchive::FStructuredArchive(FArchiveFormatterType& InFormatter)
	: Formatter(InFormatter)
#if DO_STRUCTURED_ARCHIVE_CONTAINER_CHECKS
	, bRequiresStructuralMetadata(true)
#else
	, bRequiresStructuralMetadata(InFormatter.HasDocumentTree())
#endif
{
	CurrentScope.Reserve(32);
#if DO_STRUCTURED_ARCHIVE_CONTAINER_CHECKS
	CurrentContainer.Reserve(32);
#endif
}

FStructuredArchive::~FStructuredArchive()
{
	Close();
}

FStructuredArchiveSlot FStructuredArchive::Open()
{
	check(CurrentScope.Num() == 0);
	check(!RootElementId.IsValid());
	check(!CurrentSlotElementId.IsValid());

	RootElementId = ElementIdGenerator.Generate();
	CurrentScope.Emplace(RootElementId, UE::StructuredArchive::Private::EElementType::Root);

	CurrentSlotElementId = ElementIdGenerator.Generate();

	return FStructuredArchiveSlot(FStructuredArchiveSlot::EPrivateToken{}, *this, 0, CurrentSlotElementId);
}

void FStructuredArchive::Close()
{
	SetScope(UE::StructuredArchive::Private::FSlotPosition(0, RootElementId));
}

void FStructuredArchive::EnterSlot(UE::StructuredArchive::Private::FSlotPosition Slot, bool bEnteringAttributedValue)
{
	int32                                      ParentDepth = Slot.Depth;
	UE::StructuredArchive::Private::FElementId ElementId   = Slot.ElementId;

	// If the slot being entered has attributes, enter the value slot first.
	if (ParentDepth + 1 < CurrentScope.Num() && CurrentScope[ParentDepth + 1].Id == ElementId && CurrentScope[ParentDepth + 1].Type == UE::StructuredArchive::Private::EElementType::AttributedValue)
	{
#if DO_STRUCTURED_ARCHIVE_CONTAINER_CHECKS
		checkf(!CurrentSlotElementId.IsValid() && !CurrentContainer.Top()->bAttributedValueWritten, TEXT("Attempt to serialize data into an invalid slot"));
		CurrentContainer.Top()->bAttributedValueWritten = true;
#else
		checkf(!CurrentSlotElementId.IsValid(), TEXT("Attempt to serialize data into an invalid slot"));
#endif

		SetScope(UE::StructuredArchive::Private::FSlotPosition(ParentDepth + 1, ElementId));
		Formatter.EnterAttributedValueValue();
	}
	else if (!bEnteringAttributedValue && Formatter.TryEnterAttributedValueValue())
	{
		int32 NewDepth = EnterSlotAsType(UE::StructuredArchive::Private::FSlotPosition(ParentDepth, ElementId), UE::StructuredArchive::Private::EElementType::AttributedValue);
		check(NewDepth == ParentDepth + 1);
		UE::StructuredArchive::Private::FElementId AttributedValueId = CurrentScope[NewDepth].Id;
		SetScope(UE::StructuredArchive::Private::FSlotPosition(NewDepth, AttributedValueId));
#if DO_STRUCTURED_ARCHIVE_CONTAINER_CHECKS
		CurrentContainer.Emplace(0);
#endif
	}
	else
	{
		checkf(ElementId == CurrentSlotElementId, TEXT("Attempt to serialize data into an invalid slot"));
		CurrentSlotElementId.Reset();
	}

	CurrentEnteringAttributeState = UE::StructuredArchive::Private::EEnteringAttributeState::NotEnteringAttribute;
}

int32 FStructuredArchive::EnterSlotAsType(UE::StructuredArchive::Private::FSlotPosition Slot, UE::StructuredArchive::Private::EElementType ElementType)
{
	EnterSlot(Slot, ElementType == UE::StructuredArchive::Private::EElementType::AttributedValue);

	int32 NewSlotDepth = Slot.Depth + 1;

	// If we're entering the value of an attributed slot, we need to return a depth one higher than usual, because we're
	// inside an attributed value container.
	//
	// We don't need to do adjust for attributes, because entering the attribute slot will bump the depth anyway.
	if (NewSlotDepth < CurrentScope.Num() &&
		CurrentScope[NewSlotDepth].Type == UE::StructuredArchive::Private::EElementType::AttributedValue &&
		CurrentEnteringAttributeState == UE::StructuredArchive::Private::EEnteringAttributeState::NotEnteringAttribute)
	{
		++NewSlotDepth;
	}

	CurrentScope.Emplace(Slot.ElementId, ElementType);
	return NewSlotDepth;
}

void FStructuredArchive::LeaveSlot()
{
	if (bRequiresStructuralMetadata)
	{
		switch (CurrentScope.Top().Type)
		{
		case UE::StructuredArchive::Private::EElementType::Record:
			Formatter.LeaveField();
			break;
		case UE::StructuredArchive::Private::EElementType::Array:
			Formatter.LeaveArrayElement();
#if DO_STRUCTURED_ARCHIVE_CONTAINER_CHECKS
			CurrentContainer.Top()->Index++;
#endif
			break;
		case UE::StructuredArchive::Private::EElementType::Stream:
			Formatter.LeaveStreamElement();
			break;
		case UE::StructuredArchive::Private::EElementType::Map:
			Formatter.LeaveMapElement();
#if DO_STRUCTURED_ARCHIVE_CONTAINER_CHECKS
			CurrentContainer.Top()->Index++;
#endif
			break;
		case UE::StructuredArchive::Private::EElementType::AttributedValue:
			Formatter.LeaveAttribute();
			break;
		}
	}
}

void FStructuredArchive::SetScope(UE::StructuredArchive::Private::FSlotPosition Slot)
{
	// Make sure the scope is valid
	checkf(Slot.Depth < CurrentScope.Num() && CurrentScope[Slot.Depth].Id == Slot.ElementId, TEXT("Invalid scope for writing to archive"));
	checkf(!CurrentSlotElementId.IsValid() || GetUnderlyingArchive().IsLoading(), TEXT("Cannot change scope until having written a value to the current slot"));

	// Roll back to the correct scope
	if (bRequiresStructuralMetadata)
	{
		for (int32 CurrentDepth = CurrentScope.Num() - 1; CurrentDepth > Slot.Depth; CurrentDepth--)
		{
			// Leave the current element
			const FElement& Element = CurrentScope[CurrentDepth];
			switch (Element.Type)
			{
			case UE::StructuredArchive::Private::EElementType::Record:
				Formatter.LeaveRecord();
#if DO_STRUCTURED_ARCHIVE_CONTAINER_CHECKS
				CurrentContainer.Pop(EAllowShrinking::No);
#endif
				break;
			case UE::StructuredArchive::Private::EElementType::Array:
#if DO_STRUCTURED_ARCHIVE_CONTAINER_CHECKS
				checkf(GetUnderlyingArchive().IsLoading() || CurrentContainer.Top()->Index <= CurrentContainer.Top()->Count || CurrentContainer.Top()->Count == INDEX_NONE, TEXT("Incorrect number of elements serialized in array"));
#endif
				Formatter.LeaveArray();
#if DO_STRUCTURED_ARCHIVE_CONTAINER_CHECKS
				CurrentContainer.Pop(EAllowShrinking::No);
#endif
				break;
			case UE::StructuredArchive::Private::EElementType::Stream:
				Formatter.LeaveStream();
				break;
			case UE::StructuredArchive::Private::EElementType::Map:
#if DO_STRUCTURED_ARCHIVE_CONTAINER_CHECKS
				checkf(CurrentContainer.Top()->Index == CurrentContainer.Top()->Count, TEXT("Incorrect number of elements serialized in map"));
#endif
				Formatter.LeaveMap();
#if DO_STRUCTURED_ARCHIVE_CONTAINER_CHECKS
				CurrentContainer.Pop(EAllowShrinking::No);
#endif
				break;
			case UE::StructuredArchive::Private::EElementType::AttributedValue:
				Formatter.LeaveAttributedValue();
#if DO_STRUCTURED_ARCHIVE_CONTAINER_CHECKS
				CurrentContainer.Pop(EAllowShrinking::No);
#endif
				break;
			}

			// Remove the element from the stack
			CurrentScope.RemoveAt(CurrentDepth, EAllowShrinking::No);

			// Leave the slot containing it
			LeaveSlot();
		}
	}
	else
	{
		// Remove all the top elements from the stack
		CurrentScope.RemoveAt(Slot.Depth + 1, CurrentScope.Num() - (Slot.Depth + 1));
	}
}

#endif
