// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/StructuredArchiveChildReader.h"

#if WITH_TEXT_ARCHIVE_SUPPORT

FStructuredArchiveChildReader::FStructuredArchiveChildReader(FStructuredArchiveSlot InSlot)
	: OwnedFormatter(nullptr)
	, StructuredArchive(nullptr)
{
	FStructuredArchiveFormatter* Formatter = &InSlot.StructuredArchive.Formatter;
	if (InSlot.GetUnderlyingArchive().IsTextFormat())
	{
		Formatter = OwnedFormatter = InSlot.StructuredArchive.Formatter.CreateSubtreeReader();
	}

	StructuredArchive = new FStructuredArchive(*Formatter);
	Root.Emplace(StructuredArchive->Open());
	InSlot.EnterRecord();
}

FStructuredArchiveChildReader::~FStructuredArchiveChildReader()
{
	Root.Reset();
	StructuredArchive->Close();
	delete StructuredArchive;
	StructuredArchive = nullptr;

	// If this is a text archive, we'll have created a subtree reader that our contained archive is using as 
	// its formatter. We need to clean it up now.
	if (OwnedFormatter)
	{
		delete OwnedFormatter;
		OwnedFormatter = nullptr;
	}
}

#endif
