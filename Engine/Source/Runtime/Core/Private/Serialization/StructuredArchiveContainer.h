// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_SKIP - Internal

#include "Serialization/StructuredArchive.h"
#include "Serialization/StructuredArchiveDefines.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"

#if WITH_TEXT_ARCHIVE_SUPPORT

//////////// FStructuredArchive::FContainer ////////////

#if DO_STRUCTURED_ARCHIVE_CONTAINER_CHECKS

struct FStructuredArchive::FContainer
{
	int  Index = 0;
	int  Count = 0;
	bool bAttributedValueWritten = false;

#if DO_STRUCTURED_ARCHIVE_UNIQUE_FIELD_NAME_CHECKS
	TSet<FString> KeyNames;
#endif

	explicit FContainer(int InCount)
		: Count(InCount)
	{
	}
};
#endif

#endif
