// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/StructuredArchiveFormatter.h"

//////////// FStructuredArchiveFormatter ////////////

FStructuredArchiveFormatter::~FStructuredArchiveFormatter()
{
}
