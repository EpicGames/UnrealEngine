// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/Formatters/BinaryArchiveFormatter.h"
#include "Serialization/StructuredArchive.h"
#include "Serialization/StructuredArchiveAdapters.h"
#include "Templates/MemoryOps.h"

class FArchive;

struct FStructuredArchiveFromArchive::FImpl
{
	explicit FImpl(FArchive& Ar)
		: Formatter(Ar)
		, StructuredArchive(Formatter)
		, Slot(StructuredArchive.Open())
	{
	}

	FBinaryArchiveFormatter Formatter;
	FStructuredArchive StructuredArchive;
	FStructuredArchive::FSlot Slot;
};

FStructuredArchiveFromArchive::FStructuredArchiveFromArchive(FArchive& Ar)
{
	static_assert(FStructuredArchiveFromArchive::ImplSize >= sizeof(FStructuredArchiveFromArchive::FImpl), "FStructuredArchiveFromArchive::ImplSize must fit in the size of FStructuredArchiveFromArchive::Impl");
	static_assert(FStructuredArchiveFromArchive::ImplAlignment >= alignof(FStructuredArchiveFromArchive::FImpl), "FStructuredArchiveFromArchive::ImplAlignment must be compatible with the alignment of FStructuredArchiveFromArchive::Impl");

	new (ImplStorage) FImpl(Ar);
}

FStructuredArchiveFromArchive::~FStructuredArchiveFromArchive()
{
	DestructItem((FImpl*)ImplStorage);
}

FStructuredArchive::FSlot FStructuredArchiveFromArchive::GetSlot()
{
	return ((FImpl*)ImplStorage)->Slot;
}
