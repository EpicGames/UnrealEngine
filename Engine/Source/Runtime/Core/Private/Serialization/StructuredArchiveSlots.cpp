// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/StructuredArchiveSlots.h"
#include "Serialization/StructuredArchiveContainer.h"
#include "Containers/UnrealString.h"

FArchive& UE::StructuredArchive::Private::FSlotBase::GetUnderlyingArchive() const
{
	return GetUnderlyingArchiveImpl(StructuredArchive);
}

const FArchiveState& UE::StructuredArchive::Private::FSlotBase::GetArchiveState() const
{
	return GetUnderlyingArchiveStateImpl(StructuredArchive);
}

bool FStructuredArchiveSlot::IsFilled() const
{
#if WITH_TEXT_ARCHIVE_SUPPORT
	return UE::StructuredArchive::Private::GetCurrentSlotElementIdImpl(StructuredArchive) != ElementId;
#else
	return true;
#endif
}

#if WITH_TEXT_ARCHIVE_SUPPORT

//////////// FStructuredArchiveSlot ////////////

FStructuredArchiveRecord FStructuredArchiveSlot::EnterRecord()
{
	int32 NewDepth = StructuredArchive.EnterSlotAsType(*this, UE::StructuredArchive::Private::EElementType::Record);

#if DO_STRUCTURED_ARCHIVE_CONTAINER_CHECKS
	StructuredArchive.CurrentContainer.Emplace(0);
#endif

	StructuredArchive.Formatter.EnterRecord();

	return FStructuredArchiveRecord(FStructuredArchiveRecord::EPrivateToken{}, StructuredArchive, NewDepth, ElementId);
}

FStructuredArchiveArray FStructuredArchiveSlot::EnterArray(int32& Num)
{
	int32 NewDepth = StructuredArchive.EnterSlotAsType(*this, UE::StructuredArchive::Private::EElementType::Array);

	StructuredArchive.Formatter.EnterArray(Num);

#if DO_STRUCTURED_ARCHIVE_CONTAINER_CHECKS
	StructuredArchive.CurrentContainer.Emplace(Num);
#endif

	return FStructuredArchiveArray(FStructuredArchiveArray::EPrivateToken{}, StructuredArchive, NewDepth, ElementId);
}

FStructuredArchiveStream FStructuredArchiveSlot::EnterStream()
{
	int32 NewDepth = StructuredArchive.EnterSlotAsType(*this, UE::StructuredArchive::Private::EElementType::Stream);

	StructuredArchive.Formatter.EnterStream();

	return FStructuredArchiveStream(FStructuredArchiveStream::EPrivateToken{}, StructuredArchive, NewDepth, ElementId);
}

FStructuredArchiveMap FStructuredArchiveSlot::EnterMap(int32& Num)
{
	int32 NewDepth = StructuredArchive.EnterSlotAsType(*this, UE::StructuredArchive::Private::EElementType::Map);

	StructuredArchive.Formatter.EnterMap(Num);

#if DO_STRUCTURED_ARCHIVE_CONTAINER_CHECKS
	StructuredArchive.CurrentContainer.Emplace(Num);
#endif

	return FStructuredArchiveMap(FStructuredArchiveMap::EPrivateToken{}, StructuredArchive, NewDepth, ElementId);
}

FStructuredArchiveSlot FStructuredArchiveSlot::EnterAttribute(FArchiveFieldName AttributeName)
{
	check(StructuredArchive.CurrentScope.Num() > 0);

	int32 NewDepth = Depth + 1;
	if (NewDepth >= StructuredArchive.CurrentScope.Num() || StructuredArchive.CurrentScope[NewDepth].Id != ElementId || StructuredArchive.CurrentScope[NewDepth].Type != UE::StructuredArchive::Private::EElementType::AttributedValue)
	{
		int32 NewDepthCheck = StructuredArchive.EnterSlotAsType(*this, UE::StructuredArchive::Private::EElementType::AttributedValue);
		checkSlow(NewDepth == NewDepthCheck);

		StructuredArchive.Formatter.EnterAttributedValue();

#if DO_STRUCTURED_ARCHIVE_CONTAINER_CHECKS
		StructuredArchive.CurrentContainer.Emplace(0);
#endif
	}

	StructuredArchive.CurrentEnteringAttributeState = UE::StructuredArchive::Private::EEnteringAttributeState::NotEnteringAttribute;

	UE::StructuredArchive::Private::FElementId AttributedValueId = StructuredArchive.CurrentScope[NewDepth].Id;

	StructuredArchive.SetScope(UE::StructuredArchive::Private::FSlotPosition(NewDepth, AttributedValueId));

	StructuredArchive.CurrentSlotElementId = StructuredArchive.ElementIdGenerator.Generate();

#if DO_STRUCTURED_ARCHIVE_CONTAINER_CHECKS
#if DO_STRUCTURED_ARCHIVE_UNIQUE_FIELD_NAME_CHECKS
	if (!StructuredArchive.GetUnderlyingArchive().IsLoading())
	{
		FContainer& Container = *StructuredArchive.CurrentContainer.Top();
		checkf(!Container.KeyNames.Contains(Name.Name), TEXT("Multiple attributes called '%s' serialized into attributed value"), AttributeName.Name);
		Container.KeyNames.Add(Name.Name);
	}
#endif
#endif

	StructuredArchive.Formatter.EnterAttribute(AttributeName);

	return FStructuredArchiveSlot(FStructuredArchiveSlot::EPrivateToken{}, StructuredArchive, NewDepth, StructuredArchive.CurrentSlotElementId);
}

TOptional<FStructuredArchiveSlot> FStructuredArchiveSlot::TryEnterAttribute(FArchiveFieldName AttributeName, bool bEnterWhenWriting)
{
	check(StructuredArchive.CurrentScope.Num() > 0);

	int32 NewDepth = Depth + 1;
	if (NewDepth >= StructuredArchive.CurrentScope.Num() || StructuredArchive.CurrentScope[NewDepth].Id != ElementId || StructuredArchive.CurrentScope[NewDepth].Type != UE::StructuredArchive::Private::EElementType::AttributedValue)
	{
		int32 NewDepthCheck = StructuredArchive.EnterSlotAsType(*this, UE::StructuredArchive::Private::EElementType::AttributedValue);
		checkSlow(NewDepth == NewDepthCheck);

		StructuredArchive.Formatter.EnterAttributedValue();

#if DO_STRUCTURED_ARCHIVE_CONTAINER_CHECKS
		StructuredArchive.CurrentContainer.Emplace(0);
#endif
	}

#if DO_STRUCTURED_ARCHIVE_CONTAINER_CHECKS
#if DO_STRUCTURED_ARCHIVE_UNIQUE_FIELD_NAME_CHECKS
	if (!StructuredArchive.GetUnderlyingArchive().IsLoading())
	{
		FContainer& Container = *StructuredArchive.CurrentContainer.Top();
		checkf(!Container.KeyNames.Contains(Name.Name), TEXT("Multiple attributes called '%s' serialized into attributed value"), AttributeName.Name);
		Container.KeyNames.Add(Name.Name);
	}
#endif
#endif

	UE::StructuredArchive::Private::FElementId AttributedValueId = StructuredArchive.CurrentScope[NewDepth].Id;

	StructuredArchive.SetScope(UE::StructuredArchive::Private::FSlotPosition(NewDepth, AttributedValueId));

	if (StructuredArchive.Formatter.TryEnterAttribute(AttributeName, bEnterWhenWriting))
	{
		StructuredArchive.CurrentEnteringAttributeState = UE::StructuredArchive::Private::EEnteringAttributeState::NotEnteringAttribute;

		StructuredArchive.CurrentSlotElementId = StructuredArchive.ElementIdGenerator.Generate();

		return FStructuredArchiveSlot(FStructuredArchiveSlot::EPrivateToken{}, StructuredArchive, NewDepth, StructuredArchive.CurrentSlotElementId);
	}
	else
	{
		return {};
	}
}

void FStructuredArchiveSlot::operator<< (uint8& Value)
{
	StructuredArchive.EnterSlot(*this);
	StructuredArchive.Formatter.Serialize(Value);
	StructuredArchive.LeaveSlot();
}

void FStructuredArchiveSlot::operator<< (uint16& Value)
{
	StructuredArchive.EnterSlot(*this);
	StructuredArchive.Formatter.Serialize(Value);
	StructuredArchive.LeaveSlot();
}

void FStructuredArchiveSlot::operator<< (uint32& Value)
{
	StructuredArchive.EnterSlot(*this);
	StructuredArchive.Formatter.Serialize(Value);
	StructuredArchive.LeaveSlot();
}

void FStructuredArchiveSlot::operator<< (uint64& Value)
{
	StructuredArchive.EnterSlot(*this);
	StructuredArchive.Formatter.Serialize(Value);
	StructuredArchive.LeaveSlot();
}

void FStructuredArchiveSlot::operator<< (int8& Value)
{
	StructuredArchive.EnterSlot(*this);
	StructuredArchive.Formatter.Serialize(Value);
	StructuredArchive.LeaveSlot();
}

void FStructuredArchiveSlot::operator<< (int16& Value)
{
	StructuredArchive.EnterSlot(*this);
	StructuredArchive.Formatter.Serialize(Value);
	StructuredArchive.LeaveSlot();
}

void FStructuredArchiveSlot::operator<< (int32& Value)
{
	StructuredArchive.EnterSlot(*this);
	StructuredArchive.Formatter.Serialize(Value);
	StructuredArchive.LeaveSlot();
}

void FStructuredArchiveSlot::operator<< (int64& Value)
{
	StructuredArchive.EnterSlot(*this);
	StructuredArchive.Formatter.Serialize(Value);
	StructuredArchive.LeaveSlot();
}

void FStructuredArchiveSlot::operator<< (float& Value)
{
	StructuredArchive.EnterSlot(*this);
	StructuredArchive.Formatter.Serialize(Value);
	StructuredArchive.LeaveSlot();
}

void FStructuredArchiveSlot::operator<< (double& Value)
{
	StructuredArchive.EnterSlot(*this);
	StructuredArchive.Formatter.Serialize(Value);
	StructuredArchive.LeaveSlot();
}

void FStructuredArchiveSlot::operator<< (bool& Value)
{
	StructuredArchive.EnterSlot(*this);
	StructuredArchive.Formatter.Serialize(Value);
	StructuredArchive.LeaveSlot();
}

void FStructuredArchiveSlot::operator<< (UTF32CHAR& Value)
{
	StructuredArchive.EnterSlot(*this);
	StructuredArchive.Formatter.Serialize(Value);
	StructuredArchive.LeaveSlot();
}

void FStructuredArchiveSlot::operator<< (FString& Value)
{
	StructuredArchive.EnterSlot(*this);
	StructuredArchive.Formatter.Serialize(Value);
	StructuredArchive.LeaveSlot();
}

void FStructuredArchiveSlot::operator<< (FName& Value)
{
	StructuredArchive.EnterSlot(*this);
	StructuredArchive.Formatter.Serialize(Value);
	StructuredArchive.LeaveSlot();
}

void FStructuredArchiveSlot::operator<< (UObject*& Value)
{
	StructuredArchive.EnterSlot(*this);
	StructuredArchive.Formatter.Serialize(Value);
	StructuredArchive.LeaveSlot();
}

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
void FStructuredArchiveSlot::operator<< (Verse::VCell*& Value)
{
	StructuredArchive.EnterSlot(*this);
	StructuredArchive.Formatter.Serialize(Value);
	StructuredArchive.LeaveSlot();
}
#endif

void FStructuredArchiveSlot::operator<< (FText& Value)
{
	StructuredArchive.EnterSlot(*this);
	StructuredArchive.Formatter.Serialize(Value);
	StructuredArchive.LeaveSlot();
}

void FStructuredArchiveSlot::operator<< (FWeakObjectPtr& Value)
{
	StructuredArchive.EnterSlot(*this);
	StructuredArchive.Formatter.Serialize(Value);
	StructuredArchive.LeaveSlot();
}

void FStructuredArchiveSlot::operator<< (FLazyObjectPtr& Value)
{
	StructuredArchive.EnterSlot(*this);
	StructuredArchive.Formatter.Serialize(Value);
	StructuredArchive.LeaveSlot();
}

void FStructuredArchiveSlot::operator<< (FObjectPtr& Value)
{
	StructuredArchive.EnterSlot(*this);
	StructuredArchive.Formatter.Serialize(Value);
	StructuredArchive.LeaveSlot();
}

void FStructuredArchiveSlot::operator<< (FSoftObjectPtr& Value)
{
	StructuredArchive.EnterSlot(*this);
	StructuredArchive.Formatter.Serialize(Value);
	StructuredArchive.LeaveSlot();
}

void FStructuredArchiveSlot::operator<< (FSoftObjectPath& Value)
{
	StructuredArchive.EnterSlot(*this);
	StructuredArchive.Formatter.Serialize(Value);
	StructuredArchive.LeaveSlot();
}

void FStructuredArchiveSlot::Serialize(TArray<uint8>& Data)
{
	StructuredArchive.EnterSlot(*this);
	StructuredArchive.Formatter.Serialize(Data);
	StructuredArchive.LeaveSlot();
}

void FStructuredArchiveSlot::Serialize(void* Data, uint64 DataSize)
{
	StructuredArchive.EnterSlot(*this);
	StructuredArchive.Formatter.Serialize(Data, DataSize);
	StructuredArchive.LeaveSlot();
}

//////////// FStructuredArchiveRecord ////////////

FStructuredArchiveSlot FStructuredArchiveRecord::EnterField(FArchiveFieldName Name)
{
	StructuredArchive.SetScope(*this);

	StructuredArchive.CurrentSlotElementId = StructuredArchive.ElementIdGenerator.Generate();

#if DO_STRUCTURED_ARCHIVE_CONTAINER_CHECKS
#if DO_STRUCTURED_ARCHIVE_UNIQUE_FIELD_NAME_CHECKS
	if (!StructuredArchive.GetUnderlyingArchive().IsLoading())
	{
		FContainer& Container = *Ar.CurrentContainer.Top();
		checkf(!Container.KeyNames.Contains(Name.Name), TEXT("Multiple keys called '%s' serialized into record"), Name.Name);
		Container.KeyNames.Add(Name.Name);
	}
#endif
#endif

	StructuredArchive.Formatter.EnterField(Name);

	return FStructuredArchiveSlot(FStructuredArchiveSlot::EPrivateToken{}, StructuredArchive, Depth, StructuredArchive.CurrentSlotElementId);
}

FStructuredArchiveRecord FStructuredArchiveRecord::EnterRecord(FArchiveFieldName Name)
{
	return EnterField(Name).EnterRecord();
}

FStructuredArchiveArray FStructuredArchiveRecord::EnterArray(FArchiveFieldName Name, int32& Num)
{
	return EnterField(Name).EnterArray(Num);
}

FStructuredArchiveStream FStructuredArchiveRecord::EnterStream(FArchiveFieldName Name)
{
	return EnterField(Name).EnterStream();
}

FStructuredArchiveMap FStructuredArchiveRecord::EnterMap(FArchiveFieldName Name, int32& Num)
{
	return EnterField(Name).EnterMap(Num);
}

TOptional<FStructuredArchiveSlot> FStructuredArchiveRecord::TryEnterField(FArchiveFieldName Name, bool bEnterWhenWriting)
{
	StructuredArchive.SetScope(*this);

#if DO_STRUCTURED_ARCHIVE_CONTAINER_CHECKS
#if DO_STRUCTURED_ARCHIVE_UNIQUE_FIELD_NAME_CHECKS
	if (!GetUnderlyingArchive().IsLoading())
	{
		FContainer& Container = *Ar.CurrentContainer.Top();
		checkf(!Container.KeyNames.Contains(Name.Name), TEXT("Multiple keys called '%s' serialized into record"), Name.Name);
		Container.KeyNames.Add(Name.Name);
	}
#endif
#endif

	if (StructuredArchive.Formatter.TryEnterField(Name, bEnterWhenWriting))
	{
		StructuredArchive.CurrentSlotElementId = StructuredArchive.ElementIdGenerator.Generate();
		return FStructuredArchiveSlot(FStructuredArchiveSlot::EPrivateToken{}, StructuredArchive, Depth, StructuredArchive.CurrentSlotElementId);
	}
	else
	{
		return {};
	}
}

//////////// FStructuredArchiveArray ////////////

FStructuredArchiveSlot FStructuredArchiveArray::EnterElement()
{
	StructuredArchive.SetScope(*this);

#if DO_STRUCTURED_ARCHIVE_CONTAINER_CHECKS
	checkf(StructuredArchive.CurrentContainer.Top()->Index < StructuredArchive.CurrentContainer.Top()->Count, TEXT("Serialized too many array elements"));
#endif

	StructuredArchive.CurrentSlotElementId = StructuredArchive.ElementIdGenerator.Generate();

	StructuredArchive.Formatter.EnterArrayElement();

	return FStructuredArchiveSlot(FStructuredArchiveSlot::EPrivateToken{}, StructuredArchive, Depth, StructuredArchive.CurrentSlotElementId);
}

//////////// FStructuredArchiveStream ////////////

FStructuredArchiveSlot FStructuredArchiveStream::EnterElement()
{
	StructuredArchive.SetScope(*this);

	StructuredArchive.CurrentSlotElementId = StructuredArchive.ElementIdGenerator.Generate();

	StructuredArchive.Formatter.EnterStreamElement();

	return FStructuredArchiveSlot(FStructuredArchiveSlot::EPrivateToken{}, StructuredArchive, Depth, StructuredArchive.CurrentSlotElementId);
}

//////////// FStructuredArchiveMap ////////////

FStructuredArchiveSlot FStructuredArchiveMap::EnterElement(FString& Name)
{
	StructuredArchive.SetScope(*this);

#if DO_STRUCTURED_ARCHIVE_CONTAINER_CHECKS
	checkf(StructuredArchive.CurrentContainer.Top()->Index < StructuredArchive.CurrentContainer.Top()->Count, TEXT("Serialized too many map elements"));
#endif

	StructuredArchive.CurrentSlotElementId = StructuredArchive.ElementIdGenerator.Generate();

#if DO_STRUCTURED_ARCHIVE_CONTAINER_CHECKS
#if DO_STRUCTURED_ARCHIVE_UNIQUE_FIELD_NAME_CHECKS
	if(StructuredArchive.GetUnderlyingArchive().IsSaving())
	{
		FContainer& Container = *StructuredArchive.CurrentContainer.Top();
		checkf(!Container.KeyNames.Contains(Name), TEXT("Multiple keys called '%s' serialized into record"), *Name);
		Container.KeyNames.Add(Name);
	}
#endif
#endif

	StructuredArchive.Formatter.EnterMapElement(Name);

#if DO_STRUCTURED_ARCHIVE_CONTAINER_CHECKS
#if DO_STRUCTURED_ARCHIVE_UNIQUE_FIELD_NAME_CHECKS
	if(StructuredArchive.GetUnderlyingArchive().IsLoading())
	{
		FContainer& Container = *StructuredArchive.CurrentContainer.Top();
		checkf(!Container.KeyNames.Contains(Name), TEXT("Multiple keys called '%s' serialized into record"), *Name);
		Container.KeyNames.Add(Name);
	}
#endif
#endif

	return FStructuredArchiveSlot(FStructuredArchiveSlot::EPrivateToken{}, StructuredArchive, Depth, StructuredArchive.CurrentSlotElementId);
}

#endif
