// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/VarInt.h"

#include "CoreTypes.h"
#include "Serialization/Archive.h"

int64 ReadVarIntFromArchive(FArchive& Ar)
{
	const uint64 Value = ReadVarUIntFromArchive(Ar);
	return -int64(Value & 1) ^ int64(Value >> 1);
}

void WriteVarIntToArchive(FArchive& Ar, int64 Value)
{
	WriteVarUIntToArchive(Ar, uint64((Value >> 63) ^ (Value << 1)));
}

void SerializeVarInt(FArchive& Ar, int64& Value)
{
	if (Ar.IsLoading())
	{
		Value = ReadVarIntFromArchive(Ar);
	}
	else
	{
		WriteVarIntToArchive(Ar, Value);
	}
}

uint64 ReadVarUIntFromArchive(FArchive& Ar)
{
	uint8 Buffer[9];
	Ar.Serialize(Buffer, 1);
	uint32 Size = MeasureVarUInt(Buffer);
	if (Size > 1)
	{
		Ar.Serialize(Buffer + 1, Size - 1);
	}
	return ReadVarUInt(Buffer, Size);
}

void WriteVarUIntToArchive(FArchive& Ar, uint64 Value)
{
	uint8 Buffer[9];
	const uint32 Size = WriteVarUInt(Value, Buffer);
	Ar.Serialize(Buffer, Size);
}

void SerializeVarUInt(FArchive& Ar, uint64& Value)
{
	if (Ar.IsLoading())
	{
		Value = ReadVarUIntFromArchive(Ar);
	}
	else
	{
		WriteVarUIntToArchive(Ar, Value);
	}
}
