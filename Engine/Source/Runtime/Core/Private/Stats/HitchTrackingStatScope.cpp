// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stats/HitchTrackingStatScope.h"

#if USE_LIGHTWEIGHT_STATS_FOR_HITCH_DETECTION && USE_HITCH_DETECTION
#include "AutoRTFM.h"
#include "HAL/ThreadHeartBeat.h"
#include "HAL/ThreadManager.h"
#include "Logging/LogMacros.h"

namespace UE::Stats
{
	void FHitchTrackingStatScope::ReportHitch()
	{
		if (StatString)
		{
			float Delta = float(FGameThreadHitchHeartBeat::Get().GetCurrentTime() - FGameThreadHitchHeartBeat::Get().GetFrameStartTime()) * 1000.0f;

			const uint32 CurrentThreadId = FPlatformTLS::GetCurrentThreadId();
			const bool isGT = CurrentThreadId == GGameThreadId;
			const FString& ThreadString = FThreadManager::GetThreadName(CurrentThreadId);
			FString StackString = StatString; // possibly convert from ANSICHAR

			if (!isGT && (StackString == TEXT("STAT_EventWait") || StackString == TEXT("STAT_FQueuedThread_Run_WaitForWork")))
			{
				return;
			}

			UE_LOGF(LogCore, Error, "Leaving stat scope on hitch (+%8.2fms) [%ls] %ls", Delta, *ThreadString, *StackString);
		}
	}
}

#endif // USE_LIGHTWEIGHT_STATS_FOR_HITCH_DETECTION && USE_HITCH_DETECTION
