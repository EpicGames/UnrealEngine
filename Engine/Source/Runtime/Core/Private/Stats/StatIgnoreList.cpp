// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stats/StatIgnoreList.h"

#include "CoreGlobals.h"
#include "Containers/Map.h"
#include "HAL/IConsoleManager.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/Optional.h"
#include "String/ParseTokens.h"
#include "String/Split.h"

#if UE_STATS_ALLOW_PER_THREAD_IGNORELIST
namespace UE::Stats::Private
{
	// The text representation of the thread names are what we match in the config.
	// This let's us avoid exposing ETaskTag in any public facing API and restrict which threads
	// stats can be ignored on.
	static const TMap<FStringView, ETaskTag> NamedThreadMap
	{
		{ TEXTVIEW("GameThread"), ETaskTag::EGameThread },
		{ TEXTVIEW("SlateThread"), ETaskTag::ESlateThread },
		{ TEXTVIEW("RenderingThread"), ETaskTag::ERenderingThread },
		{ TEXTVIEW("RhiThread"), ETaskTag::ERhiThread },
		{ TEXTVIEW("AsyncLoadingThread"), ETaskTag::EAsyncLoadingThread }
	};

	struct FStatIgnoreList
	{
		FORCEINLINE static FStatIgnoreList& Get()
		{
			static FStatIgnoreList Instance;
			return Instance;
		}

		void Initialize(TMap<uint32, ETaskTag>&& InIgnoreMap)
		{
			check(!bInitialized);
			IgnoreMap = MoveTemp(InIgnoreMap);
			bInitialized.store(true, std::memory_order_release);
		}

		inline bool IsStatOrGroupIgnored(uint32 StatNameHash, uint32 GroupNameHash)
		{
			if (bInitialized.load(std::memory_order_acquire))
			{
				ETaskTag CurrentThread = ETaskTag::ENone;
				if (const ETaskTag* IgnoredThreads = IgnoreMap.Find(StatNameHash))
				{
					// Note: FTaskTagScope::GetCurrentTag() is reading from a thread_local which can be quite slow
					// so we only do it if we know this stat is ignored on some thread.
					CurrentThread = FTaskTagScope::GetCurrentTag();
					return ((*IgnoredThreads & CurrentThread) == CurrentThread);
				}

				if (GroupNameHash != 0)
				{
					if (const ETaskTag* IgnoredThreads = IgnoreMap.Find(GroupNameHash))
					{
						CurrentThread = CurrentThread == ETaskTag::ENone ? FTaskTagScope::GetCurrentTag() : CurrentThread; //-V547
						return ((*IgnoredThreads & CurrentThread) == CurrentThread);
					}
				}
			}
			return false;
		}

	private:
		TMap<uint32, ETaskTag> IgnoreMap;
		std::atomic_bool bInitialized = false;
	};

} // namespace UE::Stats::Private

namespace UE::Stats
{
	void InitializeIgnoreList()
	{
		if (!GConfig)
		{
			return;
		}

		static constexpr const TCHAR* SectionName = TEXT("Stats.PerThreadIgnoreList");

		bool bIgnoreListEnabled = false;
		GConfig->GetBool(SectionName, TEXT("IgnoreListEnabled"), bIgnoreListEnabled, GEngineIni);
		if (!bIgnoreListEnabled)
		{
			return;
		}

		const auto ParseIgnoreList = [](const TCHAR* ListName, TMap<uint32, ETaskTag>& OutIgnoreMap)
		{
			TArray<FString> IgnoredEntries;
			if (GConfig->GetArray(SectionName, ListName, IgnoredEntries, GEngineIni) == 0)
			{
				return;
			}

			TMap<FStringView, ETaskTag> NameToIgnoredThreadsMap;

			for (FStringView Entry : IgnoredEntries)
			{
				FStringView Name, ThreadsString;
				if (UE::String::SplitFirst(Entry, TEXTVIEW(":"), Name, ThreadsString))
				{
					ETaskTag& IgnoredThreads = NameToIgnoredThreadsMap.FindOrAdd(Name);

					UE::String::ParseTokens(ThreadsString, TEXTVIEW("|"), [&IgnoredThreads](FStringView ThreadName)
					{
						if (const ETaskTag* Tag = Private::NamedThreadMap.Find(ThreadName.TrimStartAndEnd()))
						{
							IgnoredThreads |= *Tag;
						}
					});
				}
			}

			for (const TPair<FStringView, ETaskTag>& Pair : NameToIgnoredThreadsMap)
			{
				if (Pair.Value != ETaskTag::ENone)
				{
					const uint32 Hash = UE::HashStringFNV1a32(Pair.Key);
					OutIgnoreMap.Add(Hash, Pair.Value);
				}
			}
		};

		TMap<uint32, ETaskTag> IgnoreMap;
		// Note: we could combine these into a single list but this makes it easier to change 
		// how we store the data if we want later.
		ParseIgnoreList(TEXT("IgnoredStats"), IgnoreMap);
		ParseIgnoreList(TEXT("IgnoredGroups"), IgnoreMap);

		Private::FStatIgnoreList::Get().Initialize(MoveTemp(IgnoreMap));
	}

	bool IsStatOrGroupIgnoredOnCurrentThread(uint32 StatNameHash, uint32 GroupNameHash)
	{
		return Private::FStatIgnoreList::Get().IsStatOrGroupIgnored(StatNameHash, GroupNameHash);
	}
} // namespace UE::Stats

#endif // UE_STATS_ALLOW_PER_THREAD_IGNORELIST
