// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stats/StatsCommand.h"

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "Math/NumericLimits.h"
#include "Math/UnrealMathUtility.h"
#include "Templates/Less.h"
#include "Templates/Greater.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "UObject/NameTypes.h"
#include "Logging/LogMacros.h"
#include "CoreGlobals.h"
#include "Misc/Parse.h"
#include "Misc/ScopeLock.h"
#include "Containers/Set.h"
#include "Containers/Map.h"
#include "Misc/CoreMisc.h"
#include "Delegates/IDelegateInstance.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Stats/Stats.h"
#include "ProfilingDebugging/ProfilingHelpers.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreDelegates.h"
#include "Async/TaskGraphInterfaces.h"
#include "Misc/DefaultValueHelper.h"
#include "Tasks/Pipe.h"

#if STATS

#include "Stats/StatsData.h"
#include "Stats/StatsFile.h"

DECLARE_CYCLE_STAT(TEXT("Hitch Scan"),STAT_HitchScan,STATGROUP_StatSystem);
DECLARE_CYCLE_STAT(TEXT("HUD Group"),STAT_HUDGroup,STATGROUP_StatSystem);

DECLARE_CYCLE_STAT(TEXT("Accumulate"),STAT_Accumulate,STATGROUP_StatSystem);
DECLARE_CYCLE_STAT(TEXT("GetFlatAggregates"),STAT_GetFlatAggregates,STATGROUP_StatSystem);

static float DumpCull = 1.0f;

//Whether or not we render stats in certain modes
bool GRenderStats = true;

extern CORE_API UE::Tasks::FPipe GStatsPipe;

static TAutoConsoleVariable<int32> GCVarDumpHitchesAllThreads(
	TEXT("t.DumpHitches.AllThreads"),
	0,
	TEXT("Dump all Threads when doing stat dumphitches\n")
	TEXT(" 0: Only Game and Render Threads (default)\n")
	TEXT(" 1: All threads"),
	ECVF_RenderThreadSafe
);

void LexFromString( EStatCompareBy::Type& OutValue, const TCHAR* Buffer )
{
	OutValue = EStatCompareBy::Sum;

	if (FCString::Stricmp(Buffer, TEXT("CallCount")) == 0)
	{
		OutValue = EStatCompareBy::CallCount;
	}
	else if (FCString::Stricmp(Buffer, TEXT("Name")) == 0)
	{
		OutValue = EStatCompareBy::Name;
	}
}

/**
 * Predicate to sort stats into reverse order of definition, which historically is how people specified a preferred order.
 */
struct FGroupSort
{
	FORCEINLINE bool operator()( FStatMessage const& A, FStatMessage const& B ) const
	{
		FName GroupA = A.NameAndInfo.GetGroupName();
		FName GroupB = B.NameAndInfo.GetGroupName();
		// first sort by group
		if (GroupA == GroupB)
		{
			// cycle stats come first
			if (A.NameAndInfo.GetFlag(EStatMetaFlags::IsCycle) && !B.NameAndInfo.GetFlag(EStatMetaFlags::IsCycle))
			{
				return true;
			}
			if (!A.NameAndInfo.GetFlag(EStatMetaFlags::IsCycle) && B.NameAndInfo.GetFlag(EStatMetaFlags::IsCycle))
			{
				return false;
			}
			// then memory
			if (A.NameAndInfo.GetFlag(EStatMetaFlags::IsMemory) && !B.NameAndInfo.GetFlag(EStatMetaFlags::IsMemory))
			{
				return true;
			}
			if (!A.NameAndInfo.GetFlag(EStatMetaFlags::IsMemory) && B.NameAndInfo.GetFlag(EStatMetaFlags::IsMemory))
			{
				return false;
			}
			// otherwise, reverse order of definition
			return A.NameAndInfo.GetRawName().GetComparisonIndex() > B.NameAndInfo.GetRawName().GetComparisonIndex();
		}
		if (GroupA == NAME_None)
		{
			return false;
		}
		if (GroupB == NAME_None)
		{
			return true;
		}
		return GroupA.GetComparisonIndex() > GroupB.GetComparisonIndex();
	}
};

struct FHUDGroupManager;
struct FGroupFilter : public IItemFilter
{
	TSet<FName> const& EnabledItems;
	FString RootFilter;
	int32 RootValidCount;
	FHUDGroupManager* HudGroupManager;

	FGroupFilter(TSet<FName> const& InEnabledItems, FString InRootFilter, FHUDGroupManager* InHudGroupManager)
		: EnabledItems(InEnabledItems)
		, RootFilter(InRootFilter)
		, HudGroupManager(InHudGroupManager)
	{
		RootValidCount = RootFilter.IsEmpty() ? 1 : 0;
	}

	virtual bool Keep( FStatMessage const& Item )
	{
		const FName MessageName = Item.NameAndInfo.GetRawName();

		if (!RootFilter.IsEmpty())
		{
			EStatOperation::Type Op = Item.NameAndInfo.GetField<EStatOperation>();
			if (Op == EStatOperation::ChildrenStart && IsRoot(MessageName))
			{
				RootValidCount++;
			}
			else if (Op == EStatOperation::ChildrenEnd && IsRoot(MessageName))
			{
				RootValidCount--;
			}
		}

		return EnabledItems.Contains(MessageName) && RootValidCount;
	}
	
	~FGroupFilter()
	{
		ensure(RootValidCount >= 0 || RootFilter.IsEmpty());
	}

	bool IsRoot(const FName& MessageName) const;
};

struct FBudgetData
{
	TArray<FString> Stats;
	TSet<FName>	NonAccumulatingStats;
	TMap<FName, float> ThreadBudgetMap;
	
	/** Builds any extra meta data from the stats provided **/
	void Process()
	{
		FString ChildPrefix("-");
		for (FString& Stat : Stats)
		{
			if (Stat.RemoveFromStart(ChildPrefix))
			{
				NonAccumulatingStats.Add(FName(*Stat));
			}
		}
	}
};

FCriticalSection BudgetStatMapCS;
TMap<FName, FBudgetData> BudgetStatMapping;

/** Holds parameters used by the 'stat hier' or 'stat group ##' command. */
struct FStatParams
{
	/** Default constructor. */
	FStatParams( const TCHAR* Cmd = nullptr )
		: Group( Cmd, TEXT("group="), NAME_None )
		, SortBy( Cmd, TEXT("sortby="), EStatCompareBy::Sum )
		, Root(Cmd, TEXT("root="), NAME_None)
		, MaxHistoryFrames( Cmd, TEXT("maxhistoryframes="), 60 )
		, MaxHierarchyDepth( Cmd, TEXT("maxdepth="), 4 )
		, CullMs( Cmd, TEXT( "ms=" ), 0.2f )
		, bReset( FCString::Stristr( Cmd, TEXT("-reset") ) != nullptr )
		, bSlowMode( false )
	{}

	/**
	 * @return whether we should run stat hier reset command.
	 */
	bool ShouldReset() const
	{
		return bReset;
	}

	/** -group=[name]. */
	TParsedValueWithDefault<FName> Group;

	/** -sortby=[name|callcount|sum]. */
	TParsedValueWithDefault<EStatCompareBy::Type> SortBy;

	/** -root=[name]. */
	TParsedValueWithDefault<FName> Root;

	FName BudgetSection;

	/**
	 *	Maximum number of frames to be included in the history. 
	 *	-maxhistoryframes=[20:20-120]
	 */
	// #Stats: 2014-08-21 Replace with TParsedValueWithDefaultAndRange
	TParsedValueWithDefault<int32> MaxHistoryFrames;

	/**
	 *	Maximum depth for the hierarchy
	 * -maxdepth=16
	 */
	TParsedValueWithDefault<int32> MaxHierarchyDepth;

	/**
	 *	Threshold when start culling stats, 
	 *	if 0, disables culling
	 * -ms=5.0f
	 */
	TParsedValueWithDefault<float> CullMs;

	/** Whether to reset all collected data. */
	bool bReset;

	/** Whether to use the slow mode, which displays stats stack for the game and rendering thread. */
	bool bSlowMode;
};

/** Holds parameters used by the 'stat slow' command. */
struct FStatSlowParams : public FStatParams
{
	/** Default constructor. */
	FStatSlowParams( const TCHAR* Cmd = nullptr )
		: FStatParams( Cmd )
	{
		static FName NAME_Slow = "Slow";
		Group = TParsedValueWithDefault<FName>( nullptr, nullptr, NAME_Slow );
		CullMs = TParsedValueWithDefault<float>( Cmd, TEXT( "ms=" ), 1.0f );
		MaxHierarchyDepth = TParsedValueWithDefault<int32>( Cmd, TEXT( "maxdepth=" ), 4 );
		bSlowMode = true;
		bReset = true;
	}
};

void DumpHistoryFrame(FStatsThreadState const& StatsData, int64 TargetFrame, float InDumpCull = 0.0f, int32 MaxDepth = MAX_int32, TCHAR const* Filter = NULL)
{
	UE_LOGF(LogStats, Log, "Single Frame %lld ---------------------------------", TargetFrame);
	if (InDumpCull == 0.0f)
	{
		UE_LOGF(LogStats, Log, "Full data, use -ms=5, for example to show just the stack data with a 5ms threshhold.");
	}
	else
	{
		UE_LOGF(LogStats, Log, "Culled to %fms, use -ms=0, for all data and aggregates.", InDumpCull);
	}
	{
		UE_LOGF(LogStats, Log, "Stack ---------------");
		FRawStatStackNode Stack;
		StatsData.UncondenseStackStats(TargetFrame, Stack);
		Stack.AddSelf();
		if (InDumpCull != 0.0f)
		{
			Stack.CullByCycles( int64( InDumpCull / FPlatformTime::ToMilliseconds( 1 ) ) );		
		}
		Stack.CullByDepth( MaxDepth );
		Stack.DebugPrint(Filter);
	}
	if (InDumpCull == 0.0f)
	{
		UE_LOGF(LogStats, Log, "Inclusive aggregate stack data---------------");
		TArray<FStatMessage> Stats;
		StatsData.GetInclusiveAggregateStackStats(TargetFrame, Stats);
		Stats.Sort(FGroupSort());
		FName LastGroup = NAME_None;
		for (int32 Index = 0; Index < Stats.Num(); Index++)
		{
			FStatMessage const& Meta = Stats[Index];
			if (LastGroup != Meta.NameAndInfo.GetGroupName())
			{
				LastGroup = Meta.NameAndInfo.GetGroupName();
				UE_LOGF(LogStats, Log, "%ls", *LastGroup.ToString());
			}
			UE_LOGF(LogStats, Log, "  %ls", *FStatsUtils::DebugPrint(Meta));
		}

		UE_LOGF(LogStats, Log, "Exclusive aggregate stack data---------------");
		Stats.Empty();
		StatsData.GetExclusiveAggregateStackStats(TargetFrame, Stats);
		Stats.Sort(FGroupSort());
		LastGroup = NAME_None;
		for (int32 Index = 0; Index < Stats.Num(); Index++)
		{
			FStatMessage const& Meta = Stats[Index];
			if (LastGroup != Meta.NameAndInfo.GetGroupName())
			{
				LastGroup = Meta.NameAndInfo.GetGroupName();
				UE_LOGF(LogStats, Log, "%ls", *LastGroup.ToString());
			}
			UE_LOGF(LogStats, Log, "  %ls", *FStatsUtils::DebugPrint(Meta));
		}

		UE_LOGF(LogStats, Log, "Inclusive aggregate stack data with thread breakdown ---------------");
		Stats.Empty();
		TMap<FName, TArray<FStatMessage>> ByThread;
		StatsData.GetInclusiveAggregateStackStats(TargetFrame, Stats, nullptr, false, &ByThread);
		for(TMap<FName, TArray<FStatMessage>>::TConstIterator It(ByThread); It; ++It)
		{
			const FName ShortThreadName = FStatNameAndInfo::GetShortNameFrom(It.Key());
			UE_LOGF(LogStats, Log, "  %ls", *ShortThreadName.ToString());

			const TArray<FStatMessage>& StatMessages = It.Value();
			for(const FStatMessage& Meta : Stats)
			{
				UE_LOGF(LogStats, Log, "    %ls", *FStatsUtils::DebugPrint(Meta))
			}
		}
	}
}

void DumpNonFrame(FStatsThreadState const& StatsData, const FName OptionalGroup)
{
	if (OptionalGroup == NAME_None)
	{
		UE_LOGF(LogStats, Log, "Full non-frame data ---------------------------------");
	}
	else
	{
		UE_LOGF(LogStats, Log, "Filtered non-frame data ---------------------------------");
	}

	TArray<FStatMessage> Stats;
	for (auto It = StatsData.NotClearedEveryFrame.CreateConstIterator(); It; ++It)
	{
		if (OptionalGroup == NAME_None || OptionalGroup == It.Value().NameAndInfo.GetGroupName())
		{
			Stats.Add(It.Value());
		}
	}
	Stats.Sort(FGroupSort());
	FName LastGroup = NAME_None;
	for (int32 Index = 0; Index < Stats.Num(); Index++)
	{
		FStatMessage const& Meta = Stats[Index];
		if (LastGroup != Meta.NameAndInfo.GetGroupName())
		{
			LastGroup = Meta.NameAndInfo.GetGroupName();
			UE_LOGF(LogStats, Log, "%ls", *LastGroup.ToString());
		}
		UE_LOGF(LogStats, Log, "  %ls", *FStatsUtils::DebugPrint(Meta));
	}
}

/** Returns stats based stack as human readable string. */
static FString GetHumanReadableCallstack( const TArray<FStatNameAndInfo>& StatsStack )
{
	FString Result;

	for (int32 Index = StatsStack.Num() - 1; Index >= 0; --Index)
	{
		const FStatNameAndInfo& NameAndInfo = StatsStack[Index];

		const FString ShortName = NameAndInfo.GetShortName().GetPlainNameString();
		FString Desc = NameAndInfo.GetDescription();
		Desc.TrimStartInline();

		// For threads use the thread name, as the description contains encoded thread id.
		const FName GroupName = NameAndInfo.GetGroupName();
		if (GroupName == TEXT( "STATGROUP_Threads" ))
		{
			Desc.Empty();
		}

		if (Desc.Len() == 0)
		{
			Result += ShortName;
		}
		else
		{
			Result += Desc;
		}

		if (Index > 0)
		{
			Result += TEXT( " <- " );
		}
	}

	Result.ReplaceInline( TEXT( "STAT_" ), TEXT( "" ), ESearchCase::CaseSensitive );
	return Result;
}

/** Dumps event history if specified thread name is the as for the printing event. Removes already listed events from the history. */
void DumpEventsHistoryIfThreadValid( TArray<FEventData>& EventsHistoryForFrame, const FName ThreadName, float MinDurationToDisplay )
{
	bool bIgnoreGameAndRender = false;
	if (ThreadName == NAME_None)
	{
		bIgnoreGameAndRender = true;
	}

	UE_LOGF( LogStats, Log, "Displaying events history for %ls", *ThreadName.GetPlainNameString() );
	for( int32 Index = 0; Index < EventsHistoryForFrame.Num(); ++Index )
	{
		const FEventData& EventStats = EventsHistoryForFrame[Index];
		if (EventStats.DurationMS < MinDurationToDisplay)
		{
			break;
		}
		
		const FName EventThreadName = EventStats.WaitStackStats[0].GetShortName();
		if (EventThreadName == ThreadName || bIgnoreGameAndRender)
		{
			UE_LOGF( LogStats, Log, "Duration: %.2f MS", EventStats.DurationMS );
			UE_LOGF( LogStats, Log, " Wait   : %ls", *GetHumanReadableCallstack( EventStats.WaitStackStats ) );
			UE_LOGF( LogStats, Log, " Trigger: %ls", *GetHumanReadableCallstack( EventStats.TriggerStackStats ) );

			EventsHistoryForFrame.RemoveAt( Index--, EAllowShrinking::No);
		}	
	}
}

static FDelegateHandle DumpEventsDelegateHandle;

/** For the specified frame dumps events history to the log. */
void DumpEvents( int64 TargetFrame, float DumpEventsCullMS, bool bDisplayAllThreads )
{
	FStatsThreadState& Stats = FStatsThreadState::GetLocalState();

	// Prepare data.
	const TArray<FStatMessage> Data = Stats.GetCondensedHistory( TargetFrame );

	// Events
	struct FSortByDurationMS
	{
		FORCEINLINE bool operator()( const FEventData& A, const FEventData& B ) const
		{
			// Sort descending
			return B.DurationMS < A.DurationMS;
		}
	};
	
	TArray<FEventData> EventsHistoryForFrame;
	for( const auto& It : Stats.EventsHistory )
	{
		if (It.Value.Frame >= TargetFrame /*- STAT_FRAME_SLOP*/ && It.Value.HasValidStacks() && It.Value.DurationMS > DumpEventsCullMS)
		{
			EventsHistoryForFrame.Add( It.Value );
		}
	}

	// Don't print the header if we don't have data.
	if (EventsHistoryForFrame.Num() == 0)
	{
		return;
	}

	UE_LOGF( LogStats, Log, "----------------------------------------" );
	UE_LOGF( LogStats, Log, "Events history: Single frame %lld, greater than %2.1f ms", TargetFrame, DumpEventsCullMS );
	

	EventsHistoryForFrame.Sort( FSortByDurationMS() );

	// First print all events that wait on the game thread.
	DumpEventsHistoryIfThreadValid( EventsHistoryForFrame, NAME_GameThread, DumpEventsCullMS );

	// Second print all events that wait on the rendering thread.
	DumpEventsHistoryIfThreadValid( EventsHistoryForFrame, NAME_RenderThread, DumpEventsCullMS );

	if (bDisplayAllThreads)
	{
		// Print all the remaining events.
		DumpEventsHistoryIfThreadValid( EventsHistoryForFrame, NAME_None, DumpEventsCullMS );
	}

	UE_LOGF( LogStats, Log, "----------------------------------------" );
}

void DumpEventsOnce( int64 TargetFrame, float DumpEventsCullMS, bool bDisplayAllThreads )
{
	FStatsThreadState& Stats = FStatsThreadState::GetLocalState();
	DumpEvents( TargetFrame, DumpEventsCullMS, bDisplayAllThreads );
	StatsPrimaryEnableSubtract();
	Stats.NewFrameDelegate.Remove( DumpEventsDelegateHandle );
}

void DumpCPUSummary(FStatsThreadState const& StatsData, int64 TargetFrame)
{
	UE_LOGF(LogStats, Log, "CPU Summary: Single Frame %lld ---------------------------------", TargetFrame);

	struct FTimeInfo
	{
		int32 StartCalls;
		int32 StopCalls;
		int32 Recursion;
		FTimeInfo()
			: StartCalls(0)
			, StopCalls(0)
			, Recursion(0)
		{

		}
	};
	TMap<FName, TMap<FName, FStatMessage> > StallsPerThreads;
	TMap<FName, FTimeInfo> Timing;
	TMap<FName, FStatMessage> ThisFrameMetaData;
	TArray<FStatMessage> const& Data = StatsData.GetCondensedHistory(TargetFrame);

	static FName NAME_STATGROUP_CPUStalls("STATGROUP_CPUStalls");
	static FName Total("Total");

	int32 Level = 0;
	FName LastThread;
	for (int32 Index = 0; Index < Data.Num(); Index++)
	{
		FStatMessage const& Item = Data[Index];
		FName LongName = Item.NameAndInfo.GetRawName();

		// The description of a thread group contains the thread name marker
		const FString Desc = Item.NameAndInfo.GetDescription();
		bool bIsThread = Desc.StartsWith( FStatConstants::ThreadNameMarker );
		bool bIsStall = !bIsThread && Desc.StartsWith("CPU Stall"); // TArray<FName> StallStats/StatMessages
		
		EStatOperation::Type Op = Item.NameAndInfo.GetField<EStatOperation>();
		if ((Op == EStatOperation::ChildrenStart || Op == EStatOperation::ChildrenEnd ||  Op == EStatOperation::Leaf) && Item.NameAndInfo.GetFlag(EStatMetaFlags::IsCycle))
		{

			FTimeInfo& ItemTime = Timing.FindOrAdd(LongName);
			if (Op == EStatOperation::ChildrenStart)
			{
				ItemTime.StartCalls++;
				ItemTime.Recursion++;
				Level++;
				if (bIsThread)
				{
					LastThread = LongName;
				}
			}
			else
			{
				if (Op == EStatOperation::ChildrenEnd)
				{
					ItemTime.StopCalls++;
					ItemTime.Recursion--;
					Level--;
					if (bIsThread)
					{
						{
							FStatMessage* Result = ThisFrameMetaData.Find(LongName);
							if (!Result)
							{
								Result = &ThisFrameMetaData.Add(LongName, Item);
								Result->NameAndInfo.SetField<EStatOperation>(EStatOperation::Set);
								Result->NameAndInfo.SetFlag(EStatMetaFlags::IsPackedCCAndDuration, true);
								Result->Clear();
							}
							FStatsUtils::AccumulateStat(*Result, Item, EStatOperation::Add);
						}
						{
							FStatMessage* TotalResult = ThisFrameMetaData.Find(Total);
							if (!TotalResult)
							{
								TotalResult = &ThisFrameMetaData.Add(Total, Item);
								TotalResult->NameAndInfo.SetRawName(Total);
								TotalResult->NameAndInfo.SetField<EStatOperation>(EStatOperation::Set);
								TotalResult->NameAndInfo.SetFlag(EStatMetaFlags::IsPackedCCAndDuration, true);
								TotalResult->Clear();
							}
							FStatsUtils::AccumulateStat(*TotalResult, Item, EStatOperation::Add, true);
						}
						LastThread = NAME_None;
					}
				}
				check(!bIsStall || (!ItemTime.Recursion && LastThread != NAME_None));
				if (!ItemTime.Recursion) // doing aggregates here, so ignore misleading recursion which would be counted twice
				{
					if (LastThread != NAME_None && bIsStall)
					{
						{
							TMap<FName, FStatMessage>& ThreadStats = StallsPerThreads.FindOrAdd(LastThread);
							FStatMessage* ThreadResult = ThreadStats.Find(LongName);
							if (!ThreadResult)
							{
								ThreadResult = &ThreadStats.Add(LongName, Item);
								ThreadResult->NameAndInfo.SetField<EStatOperation>(EStatOperation::Set);
								ThreadResult->NameAndInfo.SetFlag(EStatMetaFlags::IsPackedCCAndDuration, true);
								ThreadResult->Clear();
							}
							FStatsUtils::AccumulateStat(*ThreadResult, Item, EStatOperation::Add);
						}
						{
							FStatMessage* Result = ThisFrameMetaData.Find(LastThread);
							if (!Result)
							{
								Result = &ThisFrameMetaData.Add(LastThread, Item);
								Result->NameAndInfo.SetRawName(LastThread);
								Result->NameAndInfo.SetField<EStatOperation>(EStatOperation::Set);
								Result->NameAndInfo.SetFlag(EStatMetaFlags::IsPackedCCAndDuration, true);
								Result->Clear();
							}
							FStatsUtils::AccumulateStat(*Result, Item, EStatOperation::Subtract, true);
						}
						{
							FStatMessage* TotalResult = ThisFrameMetaData.Find(Total);
							if (!TotalResult)
							{
								TotalResult = &ThisFrameMetaData.Add(Total, Item);
								TotalResult->NameAndInfo.SetRawName(Total);
								TotalResult->NameAndInfo.SetField<EStatOperation>(EStatOperation::Set);
								TotalResult->NameAndInfo.SetFlag(EStatMetaFlags::IsPackedCCAndDuration, true);
								TotalResult->Clear();
							}
							FStatsUtils::AccumulateStat(*TotalResult, Item, EStatOperation::Subtract, true);
						}
					}
				}
			}
		}
	}

	const FStatMessage* TotalStat = NULL;
	for (TMap<FName, FStatMessage>::TConstIterator ThreadIt(ThisFrameMetaData); ThreadIt; ++ThreadIt)
	{
		const FStatMessage& Item = ThreadIt.Value();
		if (Item.NameAndInfo.GetRawName() == Total)
		{
			TotalStat = &Item; 
		}
		else
		{
			UE_LOGF(LogStats, Log, "%ls%ls", FCString::Spc(2), *FStatsUtils::DebugPrint(Item));
			TMap<FName, FStatMessage>& ThreadStats = StallsPerThreads.FindOrAdd(ThreadIt.Key());
			for (TMap<FName, FStatMessage>::TConstIterator ItStall(ThreadStats); ItStall; ++ItStall)
			{
				const FStatMessage& Stall = ItStall.Value();
				UE_LOGF(LogStats, Log, "%ls%ls", FCString::Spc(4), *FStatsUtils::DebugPrint(Stall));
			}
		}
	}
	if (TotalStat)
	{
		UE_LOGF(LogStats, Log, "----------------------------------------");
		UE_LOGF(LogStats, Log, "%ls%ls", FCString::Spc(2), *FStatsUtils::DebugPrint(*TotalStat));
	}
}

static int32 HitchIndex = 0;
static double TotalHitchTime = 0.0;

static void DumpHitch(int64 Frame)
{
	// !!!CAUTION!!! 
	// Due to chain reaction of hitch reports after detecting the first hitch, the hitch detector is disabled for the next 4 frames.
	// There is no other safe method to detect if the next hitch is a real hitch or just waiting for flushing the threaded logs or waiting for the stats. 
	// So, the best way is to just wait until stats gets synchronized with the game thread.

	static int64 LastHitchFrame = -(MAX_STAT_LAG + STAT_FRAME_SLOP);
	if( LastHitchFrame + (MAX_STAT_LAG + STAT_FRAME_SLOP) > Frame )
	{
		return;
	}

	FStatsThreadState& Stats = FStatsThreadState::GetLocalState();
	SCOPE_CYCLE_COUNTER(STAT_HitchScan);

	// #todo_cycles64 - remove narrowing to 32-bit
	const double GameThreadTime = FPlatformTime::ToSeconds((uint32)Stats.GetFastThreadFrameTime(Frame, EThreadType::Game));
	const double RenderThreadTime = FPlatformTime::ToSeconds((uint32)Stats.GetFastThreadFrameTime(Frame, EThreadType::Renderer));
	const float HitchThresholdSecs = GHitchThresholdMS * 0.001f;

	if ((GameThreadTime > HitchThresholdSecs) || (RenderThreadTime > HitchThresholdSecs))
	{
		HitchIndex++;
		double ThisHitch = FMath::Max<double>(GameThreadTime, RenderThreadTime) * 1000.0;
		TotalHitchTime += ThisHitch;
		UE_LOGF(LogStats, Log, "------------------Thread Hitch %d, Frame %lld  %6.1fms ---------------", HitchIndex, Frame, ThisHitch);
		FRawStatStackNode Stack;
		Stats.UncondenseStackStats(Frame, Stack);
		Stack.AddNameHierarchy();
		Stack.AddSelf();
		
		const float MinTimeToReportInSecs = DumpCull / 1000.0f;
		const int64 MinCycles = int64(MinTimeToReportInSecs / FPlatformTime::GetSecondsPerCycle());
		FRawStatStackNode* GameThread = NULL;
		FRawStatStackNode* RenderThread = NULL;
		bool bDumpAllThreads = GCVarDumpHitchesAllThreads.GetValueOnAnyThread() != 0;
		for( auto ChildIter = Stack.Children.CreateConstIterator(); ChildIter; ++ChildIter )
		{
			const FName ThreadName = ChildIter.Value()->Meta.NameAndInfo.GetShortName();

			if( ThreadName == FName( NAME_GameThread ) )
			{
				GameThread = ChildIter.Value();
				UE_LOGF( LogStats, Log, "------------------ Game Thread %.2fms", GameThreadTime * 1000.0f );
				GameThread->CullByCycles( MinCycles );
				GameThread->DebugPrint(nullptr, 127);
			}
			else if( ThreadName == FName( NAME_RenderThread ) )
			{
				RenderThread = ChildIter.Value();
				UE_LOGF( LogStats, Log, "------------------ Render Thread (%ls) %.2fms", *RenderThread->Meta.NameAndInfo.GetRawName().ToString(), RenderThreadTime * 1000.0f );
				RenderThread->CullByCycles( MinCycles );
				RenderThread->DebugPrint(nullptr, 127);
			}
			else if (bDumpAllThreads)
			{
				FRawStatStackNode* OtherThread = ChildIter.Value();
				UE_LOGF(LogStats, Log, "------------------ OTHER Thread (%ls)", *OtherThread->Meta.NameAndInfo.GetRawName().ToString());
				OtherThread->CullByCycles(MinCycles);
				OtherThread->DebugPrint();
			}
		}

		if( !GameThread )
		{
			UE_LOGF( LogStats, Warning, "No game thread?!" );
		}

		if( !RenderThread )
		{
			UE_LOGF( LogStats, Warning, "No render thread." );
		}

		LastHitchFrame = Frame;

		// Display events, but only the large ones.
		DumpEvents( Frame, 1.0f, false );
	}
}

#endif

static bool HandleToggleCommandBroadcast(const FName& InStatName, bool& bOutCurrentEnabled, bool& bOutOthersEnabled)
{
	// !!! Not thread-safe, calling game thread code from the stats thread. !!!

	bOutCurrentEnabled = true;
	bOutOthersEnabled = false;

	// Check to see if all stats have been disabled... 
	static const FName NAME_NoGroup = FName("STATGROUP_None");
	if (InStatName == NAME_NoGroup)
	{
		// Iterate through all enabled groups.
		FCoreDelegates::StatDisableAll.Broadcast(true);

		return false;
	}

	// Check to see if/how this is already enabled.. (default to these incase it's not bound)
	FString StatString = InStatName.ToString();
	StatString.RemoveFromStart("STATGROUP_");
	if (FCoreDelegates::StatCheckEnabled.IsBound())
	{
		FCoreDelegates::StatCheckEnabled.Broadcast(*StatString, bOutCurrentEnabled, bOutOthersEnabled);
		if (!bOutCurrentEnabled)
		{
			FCoreDelegates::StatEnabled.Broadcast(*StatString);
		}
		else
		{
			FCoreDelegates::StatDisabled.Broadcast(*StatString);
		}
	}

	return true;
}

#if STATS

void FLatestGameThreadStatsData::NewData(FGameThreadStatsData* Data)
{
	delete Latest;
	Latest = Data;
}

FLatestGameThreadStatsData& FLatestGameThreadStatsData::Get()
{
	static FLatestGameThreadStatsData Singleton;
	return Singleton;
}

FStatGroupGameThreadNotifier& FStatGroupGameThreadNotifier::Get()
{
	static FStatGroupGameThreadNotifier Singleton;
	return Singleton;
}

struct FInternalGroup
{
	/** Initialization constructor. */
	FInternalGroup(const FName InGroupName, const FName InGroupCategory, const EStatDisplayMode::Type InDisplayMode, bool InCompareByName, TSet<FName>& InEnabledItems, const FString& InGroupDescription, TMap<FName, float>* InThreadBudgetMap = nullptr, TSet<FName>* InBudgetIgnore = nullptr)
		: GroupName( InGroupName )
		, GroupCategory(InGroupCategory)
		, GroupDescription( InGroupDescription )
		, DisplayMode( InDisplayMode )
		, bCompareByName( InCompareByName )
		, SortOrder( NextCounter++ )
	{
		// To avoid copy.
		Exchange( EnabledItems, InEnabledItems );
		
		if(InThreadBudgetMap)
		{
			Exchange(ThreadBudgetMap, *InThreadBudgetMap);	//avoid copy
		}

		if (InBudgetIgnore)
		{
			Exchange(BudgetIgnoreStats, *InBudgetIgnore);	//avoid copy
		}
	}

	/** Set of elements which should be included in this group stats. */
	TSet<FName> EnabledItems;

	/** Name of this stat group. */
	FName GroupName;

	/** Category of this stat group. */
	FName GroupCategory;

	/** Description of this stat group. */
	FString GroupDescription;

	/** If budget mode is used, this is the expected cost of the stats in the group added up. */
	TMap<FName, float> ThreadBudgetMap;

	/** If budget mode is used, these are the stats that we display, but ignore during summation */
	TSet<FName> BudgetIgnoreStats;

	/** Display mode for this group. */
	EStatDisplayMode::Type DisplayMode;

	/** Sorting mode for this group. */
	bool bCompareByName;

	/** Counter used to render stat groups in the order they were enabled. */
	const uint64 SortOrder;
	static uint64 NextCounter;
};

uint64 FInternalGroup::NextCounter = 0;

/** Stats for the particular frame. */
struct FHudFrame
{
	TArray<FStatMessage> InclusiveAggregate;
	TArray<FStatMessage> ExclusiveAggregate;
	TArray<FStatMessage> NonStackStats;
	FRawStatStackNode HierarchyInclusive;
	TMap<FName, TArray<FStatMessage>> InclusiveAggregateThreadBreakdown;
};

struct FHUDGroupManager 
{
	/** Contains all enabled groups. */
	TMap<FName,FInternalGroup> EnabledGroups;

	/** Contains all history frames. */
	TMap<int64,FHudFrame> History;

	/** Cache for filters that rely on root substring */
	TMap<FName, bool> RootFilterCache;

	/** Root stat stack for all frames, it's accumulating all the time, but can be reset with a command 'stat hier -reset'. */
	FRawStatStackNode TotalHierarchyInclusive;
	
	/** Flat array of messages, it's accumulating all the time, but can be reset with a command 'stat hier -reset'. */
	TArray<FStatMessage> TotalAggregateInclusive;
	TArray<FStatMessage> TotalNonStackStats;
	TMap<FName, TArray<FStatMessage>> TotalAggregateInclusiveThreadBreakdown;

	/** Root stat stack for history frames, by default it's for the last 20 frames. */
	FComplexRawStatStackNode AggregatedHierarchyHistory;
	TArray<FComplexStatMessage> AggregatedFlatHistory;
	TMap<FName, TArray<FComplexStatMessage>> AggregatedFlatHistoryThreadBreakdown;
	TArray<FComplexStatMessage> AggregatedNonStackStatsHistory;

	/** Copy of the stat group command parameters. */
	FStatParams Params;
	
	/** Number of frames for the root stat stack. */
	int32 NumTotalStackFrames;

	/** Index of the latest frame. */
	int64 LatestFrame;

	/** Reference to the stats state. */
	FStatsThreadState const& Stats;

	/** Whether it's enabled or not. */
	bool bEnabled;

	/** NewFrame delegate handle */
	FDelegateHandle NewFrameDelegateHandle;

	/** Default constructor. */
	FHUDGroupManager(FStatsThreadState const& InStats)
		: NumTotalStackFrames(0)
		, LatestFrame(-2)
		, Stats(InStats)
		, bEnabled(false)
	{
	}

	/** Handles hier or group command. */
	void HandleCommand( const FStatParams& InParams, const bool bHierarchy )
	{
		bool bCurrentEnabled, bOthersEnabled;

		bool bResetData = false;
		if (Params.bSlowMode != InParams.bSlowMode)
		{
			bResetData = true;
		}

		if (Params.BudgetSection != InParams.BudgetSection)
		{
			bResetData = true;
		}

		Params = InParams;
		Params.bReset = bResetData;

		RootFilterCache.Empty();

		if( Params.ShouldReset() )
		{
			// Disable only stats groups, leave the fake FPS, Unit group untouched.
			for (const auto& It : EnabledGroups)
			{
				HandleToggleCommandBroadcast( It.Key, bCurrentEnabled, bOthersEnabled );
			}

			EnabledGroups.Empty();
			History.Empty();
			NumTotalStackFrames = 0;
		}

		ResizeFramesHistory( Params.MaxHistoryFrames.Get() );

		const FName MaybeGroupFName = FName(*(FString("STATGROUP_") + Params.Group.Get().GetPlainNameString()));
		const bool bResults = HandleToggleCommandBroadcast( MaybeGroupFName, bCurrentEnabled, bOthersEnabled );
		if (!bResults)
		{
			// Remove all groups.
			EnabledGroups.Empty();
		}
		else
		{
			// Is this a group stat (as opposed to a simple stat?)
			const bool bGroupStat = Stats.Groups.Contains(MaybeGroupFName);
			if (bGroupStat)
			{
				// Is this group stat currently enabled?
				if (FInternalGroup* InternalGroup = EnabledGroups.Find(MaybeGroupFName))
				{
					// If this was only being used by the current viewport, remove it
					if (bCurrentEnabled && !bOthersEnabled)
					{
						if ((InternalGroup->DisplayMode & EStatDisplayMode::Hierarchical) && !bHierarchy)
						{
							InternalGroup->DisplayMode = EStatDisplayMode::Flat;
						}
						else if ((InternalGroup->DisplayMode & EStatDisplayMode::Flat) && bHierarchy)
						{
							InternalGroup->DisplayMode = EStatDisplayMode::Hierarchical;
						}
						else
						{
							EnabledGroups.Remove(MaybeGroupFName);
							NumTotalStackFrames = 0;
						}
					}
				}
				else
				{
					// If InternalGroup is null, it shouldn't be being used by any viewports					
					TSet<FName> EnabledItems;
					GetStatsForGroup(EnabledItems, MaybeGroupFName);

					const FStatMessage& Group = Stats.ShortNameToLongName.FindChecked(MaybeGroupFName);
					const FName GroupCategory = Group.NameAndInfo.GetGroupCategory();
					const FString GroupDescription = Group.NameAndInfo.GetDescription();
					const bool bSortByName = Group.NameAndInfo.GetSortByName();

					EnabledGroups.Add(MaybeGroupFName, FInternalGroup(MaybeGroupFName, GroupCategory, bHierarchy ? EStatDisplayMode::Hierarchical : EStatDisplayMode::Flat, bSortByName, EnabledItems, GroupDescription));
				}
			}
			else if (Params.bSlowMode)
			{
				const bool bEnabledSlowMode = EnabledGroups.Contains( MaybeGroupFName );
				if (bEnabledSlowMode)
				{
					EnabledGroups.Remove( MaybeGroupFName );
					NumTotalStackFrames = 0;
				}
				else
				{
					TSet<FName> EmptySet = TSet<FName>();
					EnabledGroups.Add( MaybeGroupFName, FInternalGroup( MaybeGroupFName, NAME_None, EStatDisplayMode::Hierarchical, false, EmptySet, TEXT( "Hierarchy for game and render" ) ) );
				}			
			}
			else if(Params.BudgetSection != NAME_None)
			{
				const bool bEnabledBudgetMode = EnabledGroups.Num() > 0;
				if (bEnabledBudgetMode)
				{
					for (const auto& It : EnabledGroups)
					{
						HandleToggleCommandBroadcast(It.Key, bCurrentEnabled, bOthersEnabled);
					}

					EnabledGroups.Empty();
					NumTotalStackFrames = 0;
				}
				else
				{
					TMap<FName, float> ThreadBudgetMap;
					TArray<FName> StatShortNames;
					TSet<FName> NonAccumulatingStats;
					{
						FScopeLock BudgetLock(&BudgetStatMapCS);
						if(FBudgetData* BudgetData = BudgetStatMapping.Find(Params.BudgetSection))
						{
							for(const FString& StatEntry : BudgetData->Stats)
							{
								StatShortNames.Add(FName(*StatEntry));
							}

							NonAccumulatingStats = BudgetData->NonAccumulatingStats;
							ThreadBudgetMap = BudgetData->ThreadBudgetMap;
						}
					}
					
					{
						TSet<FName> EmptyStatSet; // see LazyInitBudgetGroup
						FName BudgetGroupName(Params.BudgetSection);
						EnabledGroups.Add(BudgetGroupName, FInternalGroup(Params.BudgetSection, NAME_None, EStatDisplayMode::Flat, false, EmptyStatSet, TEXT("Budget"), &ThreadBudgetMap, &NonAccumulatingStats));
						HandleToggleCommandBroadcast( BudgetGroupName, bCurrentEnabled, bOthersEnabled );
					}
				}
			}
		}

		if( EnabledGroups.Num() && !bEnabled )
		{
			bEnabled = true;
			NewFrameDelegateHandle = Stats.NewFrameDelegate.AddRaw( this, &FHUDGroupManager::NewFrame );
			StatsPrimaryEnableAdd();
		}
		else if( !EnabledGroups.Num() && bEnabled )
		{
			Stats.NewFrameDelegate.Remove( NewFrameDelegateHandle );
			StatsPrimaryEnableSubtract();
			bEnabled = false;

			DECLARE_CYCLE_STAT(TEXT("FSimpleDelegateGraphTask.StatsToGame"),
				STAT_FSimpleDelegateGraphTask_StatsToGame,
				STATGROUP_TaskGraphTasks);

			FSimpleDelegateGraphTask::CreateAndDispatchWhenReady
			(
				FSimpleDelegateGraphTask::FDelegate::CreateRaw(&FLatestGameThreadStatsData::Get(), &FLatestGameThreadStatsData::NewData, (FGameThreadStatsData*)nullptr),
				GET_STATID(STAT_FSimpleDelegateGraphTask_StatsToGame), nullptr, ENamedThreads::GameThread
			);
		}
	}

	/** Convert short name stats to long name. Delayed so the ShortNameToLongName map gets populated with more stats. */
	void LazyInitBudgetGroup()
	{
		FName BudgetGroupName(Params.BudgetSection);
		if (FInternalGroup* InternalGroup = EnabledGroups.Find(BudgetGroupName))
		{
			if (InternalGroup->EnabledItems.Num() > 0)
			{				
				return; // skip if already initialized
			}

			TArray<FName> StatShortNames;
			FScopeLock BudgetLock(&BudgetStatMapCS);
			if (FBudgetData* BudgetData = BudgetStatMapping.Find(Params.BudgetSection))
			{
				for (const FString& StatEntry : BudgetData->Stats)
				{
					StatShortNames.Add(FName(*StatEntry));
				}
			}
			GetStatsForNames(InternalGroup->EnabledItems, StatShortNames);
		}
	}

	void ResizeFramesHistory( int32 MaxFrames )
	{
		History.Empty(MaxFrames + 1);
	}

	void LinearizeStackForItems( const FComplexRawStatStackNode& StackNode, const TSet<FName>& EnabledItems, TArray<FComplexStatMessage>& out_HistoryStack, TArray<int32>& out_Indentation, int32 Depth )
	{
		const bool bToBeAdded = EnabledItems.Contains( StackNode.ComplexStat.NameAndInfo.GetRawName() );
		if( bToBeAdded )
		{
			out_HistoryStack.Add( StackNode.ComplexStat );
			out_Indentation.Add( Depth );
		}

		for( auto It = StackNode.Children.CreateConstIterator(); It; ++It )
		{
			const FComplexRawStatStackNode& Child = *It.Value();
			LinearizeStackForItems( Child, EnabledItems, out_HistoryStack, out_Indentation, Depth+1 ); 
		}
	}

	void LinearizeSlowStackForItems( const FComplexRawStatStackNode& StackNode, TArray<FComplexStatMessage>& out_HistoryStack, TArray<int32>& out_Indentation, int32 Depth )
	{
		// Ignore first call, this is the thread root.
		const bool bToBeAdded = Depth > 0;// StackNode.ComplexStat.GetShortName() != FStatConstants::NAME_ThreadRoot;
		if (bToBeAdded)
		{
			out_HistoryStack.Add( StackNode.ComplexStat );
			out_Indentation.Add( Depth );
		}

		for (auto It = StackNode.Children.CreateConstIterator(); It; ++It)
		{
			const FComplexRawStatStackNode& Child = *It.Value();
			LinearizeSlowStackForItems( Child, out_HistoryStack, out_Indentation, Depth + 1 );
		}
	}

	void NewFrame(int64 TargetFrame)
	{
		SCOPE_CYCLE_COUNTER(STAT_HUDGroup);
		check(bEnabled);

		// Add a new frame to the history.
		FHudFrame& NewFrame = History.FindOrAdd( TargetFrame );

		FName RootName = Params.Root.Get();
		FString RootString = (RootName == NAME_None) ? FString() : RootName.ToString();

		const bool bUseSlowMode = Params.bSlowMode;
		const bool bUseBudgetMode = Params.BudgetSection != NAME_None;

		if (bUseSlowMode)
		{
			// Only for game thread and rendering thread.
			Stats.UncondenseStackStats( TargetFrame, NewFrame.HierarchyInclusive, nullptr, nullptr );

			for (auto ChildIt = NewFrame.HierarchyInclusive.Children.CreateIterator(); ChildIt; ++ChildIt)
			{
				const FName ThreadName = ChildIt.Value()->Meta.NameAndInfo.GetShortName();

				if (ThreadName == NAME_GameThread)
				{
					continue;
				}
				else if (ThreadName == NAME_RenderThread)
				{
					continue;
				}

				delete ChildIt.Value();
				ChildIt.RemoveCurrent();
			}
		}
		else
		{
			if (bUseBudgetMode)
			{
				LazyInitBudgetGroup();
			}

			TSet<FName> HierEnabledItems;
			for( auto It = EnabledGroups.CreateIterator(); It; ++It )
			{
				if(It.Value().ThreadBudgetMap.Num() == 0)
				{
					QUICK_SCOPE_CYCLE_COUNTER(STAT_GetStatsForGroup_EveryFrame);
					GetStatsForGroup(It.Value().EnabledItems, It.Key());
				}

				HierEnabledItems.Append(It.Value().EnabledItems);
			}
		
			
			FGroupFilter Filter(HierEnabledItems, RootString, this);

			// Generate root stats stack for current frame.
			Stats.UncondenseStackStats( TargetFrame, NewFrame.HierarchyInclusive, &Filter, &NewFrame.NonStackStats );

			{
				SCOPE_CYCLE_COUNTER(STAT_GetFlatAggregates);
				Stats.GetInclusiveAggregateStackStats( TargetFrame, NewFrame.InclusiveAggregate, &Filter, false, &NewFrame.InclusiveAggregateThreadBreakdown );
				Stats.GetExclusiveAggregateStackStats( TargetFrame, NewFrame.ExclusiveAggregate, &Filter, false );

				//Merge all task graph stats into 1
				const TCHAR* TaskGraphWorkerPrefixes[] =
				{
					TEXT("TaskGraphThread"),
					TEXT("Foreground Worker #"),
					TEXT("Background Worker #")
				};
				TArray<FStatMessage> MergedTaskGraphThreads;
				for(TMap<FName, TArray<FStatMessage>>::TIterator It(NewFrame.InclusiveAggregateThreadBreakdown); It; ++It)
				{
					const FString ThreadName = FStatNameAndInfo::GetShortNameFrom(It.Key()).ToString();
					for (const TCHAR* Prefix : TaskGraphWorkerPrefixes)
					{
						if (ThreadName.StartsWith(Prefix))
						{
							FStatsUtils::AddMergeStatArray(MergedTaskGraphThreads, It.Value());
							It.RemoveCurrent();
							break;
						}
					}
				}
				
				if(MergedTaskGraphThreads.Num())
				{
					NewFrame.InclusiveAggregateThreadBreakdown.Add(FName("MergedTaskGraphThreads"), MergedTaskGraphThreads);
				}
			}
		}

		NewFrame.HierarchyInclusive.AddSelf();
		// To get the good performance we must pre-filter the results.
		NewFrame.HierarchyInclusive.CullByCycles( int64( 0.001f / FPlatformTime::GetSecondsPerCycle() * 0.1f ) );
		NewFrame.HierarchyInclusive.CullByDepth( Params.MaxHierarchyDepth.Get() );

		// Aggregate hierarchical stats.
		if( NumTotalStackFrames == 0 )
		{
			TotalHierarchyInclusive = NewFrame.HierarchyInclusive;
		}
		else
		{
			TotalHierarchyInclusive.MergeAdd( NewFrame.HierarchyInclusive );
		}
		
		// Aggregate flat stats.
		if( NumTotalStackFrames == 0 )
		{
			TotalAggregateInclusive = NewFrame.InclusiveAggregate;
			TotalAggregateInclusiveThreadBreakdown = NewFrame.InclusiveAggregateThreadBreakdown;
		}
		else
		{
			FStatsUtils::AddMergeStatArray( TotalAggregateInclusive, NewFrame.InclusiveAggregate );
			for(TMap<FName, TArray<FStatMessage>>::TConstIterator It(NewFrame.InclusiveAggregateThreadBreakdown); It; ++It)
			{
				FStatsUtils::AddMergeStatArray(TotalAggregateInclusiveThreadBreakdown.FindOrAdd(It.Key()), It.Value());
			}
		}

		// Aggregate non-stack stats.
		if( NumTotalStackFrames == 0 )
		{
			TotalNonStackStats = NewFrame.NonStackStats;
		}
		else
		{
			FStatsUtils::AddMergeStatArray( TotalNonStackStats, NewFrame.NonStackStats );
		}
		NumTotalStackFrames ++;
		
		/** Not super efficient, but allows to sort different stat data types. */
		struct FStatValueComparer
		{
			FORCEINLINE_STATS bool operator()( const FStatMessage& A, const FStatMessage& B ) const
			{
				// We assume that stat data type may be only int64 or double.
				const EStatDataType::Type DataTypeA = A.NameAndInfo.GetField<EStatDataType>();
				const EStatDataType::Type DataTypeB = B.NameAndInfo.GetField<EStatDataType>();
				const bool bAIsInt = DataTypeA == EStatDataType::ST_int64;
				const bool bBIsInt = DataTypeB == EStatDataType::ST_int64;
				const bool bAIsDbl = DataTypeA == EStatDataType::ST_double;
				const bool bBIsDbl = DataTypeB == EStatDataType::ST_double;

				const double ValueA = bAIsInt ? double(A.GetValue_int64()) : A.GetValue_double();
				const double ValueB = bBIsInt ? double(B.GetValue_int64()) : B.GetValue_double();

				return ValueA == ValueB ? FStatNameComparer<FStatMessage>()(A,B) : ValueA > ValueB;
			}
		};
		struct FStatNameAndValueComparer
		{
			FStatNameComparer<FStatMessage> NameComparer;
			FStatValueComparer ValueComparer;

			FORCEINLINE_STATS bool operator()( const FStatMessage& A, const FStatMessage& B ) const
			{
				const bool bSortAByName = A.NameAndInfo.GetSortByName();
				const bool bSortBByName = B.NameAndInfo.GetSortByName();
				// If both use the same sort method, use the appropriate comparer, otherwise sort by name goes first.
				if (bSortAByName)
				{
					return bSortBByName ? NameComparer(A, B) : true;
				}
				else
				{
					return bSortBByName ? false : ValueComparer(A, B);
				}
			}
		};

		if(!bUseBudgetMode)	//In budget mode we do not sort since we want to maintain hierarchy
		{
			// Sort total history stats by the specified item.
			EStatCompareBy::Type StatCompare = Params.SortBy.Get();

 			if (StatCompare == EStatCompareBy::Sum)
			{
				TotalHierarchyInclusive.Sort(FStatDurationComparer<FRawStatStackNode>());
				TotalAggregateInclusive.Sort(FStatDurationComparer<FStatMessage>());
				for(TMap<FName, TArray<FStatMessage>>::TIterator It(TotalAggregateInclusiveThreadBreakdown); It; ++It)
				{
					It.Value().Sort(FStatDurationComparer<FStatMessage>());
				}
			}
			else if (StatCompare == EStatCompareBy::CallCount)
			{
				TotalHierarchyInclusive.Sort(FStatCallCountComparer<FRawStatStackNode>());
				TotalAggregateInclusive.Sort(FStatCallCountComparer<FStatMessage>());
				for (TMap<FName, TArray<FStatMessage>>::TIterator It(TotalAggregateInclusiveThreadBreakdown); It; ++It)
				{
					It.Value().Sort(FStatCallCountComparer<FStatMessage>());
				}
			}
			else if (StatCompare == EStatCompareBy::Name)
			{
				TotalHierarchyInclusive.Sort(FStatNameComparer<FRawStatStackNode>());
				TotalAggregateInclusive.Sort(FStatNameComparer<FStatMessage>());
				for (TMap<FName, TArray<FStatMessage>>::TIterator It(TotalAggregateInclusiveThreadBreakdown); It; ++It)
				{
					It.Value().Sort(FStatNameComparer<FStatMessage>());
				}
			}

			// For the non stack stats, their can be any mix of sort by name or by value.
			int32 NumGroupsUsingCompareByName = 0;
			for (auto It = EnabledGroups.CreateIterator(); It; ++It)
			{
				if (It.Value().bCompareByName)
				{
					++NumGroupsUsingCompareByName;
				}
			}

			// If all groups enforce sort by name, or that the current settings is sort by name.
			if (NumGroupsUsingCompareByName == EnabledGroups.Num() || StatCompare == EStatCompareBy::Name)
			{
				TotalNonStackStats.Sort(FStatNameComparer<FStatMessage>());
			}
			// If all groups must use a sort by value.
			else if (NumGroupsUsingCompareByName == 0)
			{
				TotalNonStackStats.Sort(FStatValueComparer());
			}
			// Otherwise do a mixed sort, more expensive because it processes each stat strings to figure out if it requires sort by name.
			else
			{
				TotalNonStackStats.Sort(FStatNameAndValueComparer());
			}
		}

		// We want contiguous frames only.
		if( TargetFrame - LatestFrame > 1 ) 
		{
			ResizeFramesHistory( Params.MaxHistoryFrames.Get() );
		}

		RemoveFramesOutOfHistory(TargetFrame);

		const int32 NumFrames = History.Num();
		check(NumFrames <= Params.MaxHistoryFrames.Get());
		if( NumFrames > 0 )
		{
			FGameThreadStatsData* ToGame = new FGameThreadStatsData(false, GRenderStats);
			ToGame->RootFilter = RootString;

			// Copy the total stats stack to the history stats stack and clear all nodes' data and set data type to none.
			// Called to maintain the hierarchy.
			AggregatedHierarchyHistory.CopyNameHierarchy( TotalHierarchyInclusive );

			// Copy flat-stack stats
			AggregatedFlatHistory.Reset( TotalAggregateInclusive.Num() );
			for( int32 Index = 0; Index < TotalAggregateInclusive.Num(); ++Index )
			{
				const FStatMessage& StatMessage = TotalAggregateInclusive[Index];
				AggregatedFlatHistory.Emplace(StatMessage);
			}

			// Copy flat-stack stats by thread
			AggregatedFlatHistoryThreadBreakdown.Reset();
			for(TMap<FName, TArray<FStatMessage>>::TConstIterator It(TotalAggregateInclusiveThreadBreakdown); It; ++It)
			{
				TArray<FComplexStatMessage>& AggregatedFlatHistoryThreadBreakdownArray = AggregatedFlatHistoryThreadBreakdown.Add(It.Key());
				for (const FStatMessage& StatMessage : It.Value())
				{
					AggregatedFlatHistoryThreadBreakdownArray.Emplace(StatMessage);
				}
			}

			// Copy non-stack stats
			AggregatedNonStackStatsHistory.Reset( TotalNonStackStats.Num() );
			for( int32 Index = 0; Index < TotalNonStackStats.Num(); ++Index )
			{
				const FStatMessage& StatMessage = TotalNonStackStats[Index];
				AggregatedNonStackStatsHistory.Emplace(StatMessage);
			}
			
			// Accumulate hierarchy, flat and non-stack stats.
			for( auto FrameIt = History.CreateConstIterator(); FrameIt; ++FrameIt )
			{
				SCOPE_CYCLE_COUNTER(STAT_Accumulate);
				const FHudFrame& Frame = FrameIt.Value();

				AggregatedHierarchyHistory.MergeAddAndMinMax( Frame.HierarchyInclusive );

				FComplexStatUtils::MergeAddAndMinMaxArray( AggregatedFlatHistory, Frame.InclusiveAggregate, EComplexStatField::IncSum, EComplexStatField::IncMax, EComplexStatField::IncMin );

				for (TMap<FName, TArray<FStatMessage>>::TConstIterator It(Frame.InclusiveAggregateThreadBreakdown); It; ++It)
				{
					FComplexStatUtils::MergeAddAndMinMaxArray( AggregatedFlatHistoryThreadBreakdown.FindChecked(It.Key()), It.Value(), EComplexStatField::IncSum, EComplexStatField::IncMax, EComplexStatField::IncMin);
				}

				FComplexStatUtils::MergeAddAndMinMaxArray( AggregatedFlatHistory, Frame.ExclusiveAggregate, EComplexStatField::ExcSum, EComplexStatField::ExcMax, EComplexStatField::ExcMin);
				FComplexStatUtils::MergeAddAndMinMaxArray( AggregatedNonStackStatsHistory, Frame.NonStackStats, EComplexStatField::IncSum, EComplexStatField::IncMax, EComplexStatField::IncMin);
			}

			// Divide stats to get average values.
			AggregatedHierarchyHistory.Divide( NumFrames );
			AggregatedHierarchyHistory.CopyExclusivesFromSelf();
			if (Params.CullMs.Get() != 0.0f)
			{
				AggregatedHierarchyHistory.CullByCycles( int64( Params.CullMs.Get() / FPlatformTime::ToMilliseconds( 1 ) ) );
			}
			AggregatedHierarchyHistory.CullByDepth( Params.MaxHierarchyDepth.Get() );

			// Make sure the game thread is first.
			AggregatedHierarchyHistory.Children.KeySort(FNameLexicalLess());

			FComplexStatUtils::DiviveStatArray( AggregatedFlatHistory, NumFrames, EComplexStatField::IncSum, EComplexStatField::IncAve );
			FComplexStatUtils::DiviveStatArray( AggregatedFlatHistory, NumFrames, EComplexStatField::ExcSum, EComplexStatField::ExcAve );

			for(TMap<FName, TArray<FComplexStatMessage>>::TIterator It(AggregatedFlatHistoryThreadBreakdown); It; ++It)
			{
				FComplexStatUtils::DiviveStatArray(It.Value(), NumFrames, EComplexStatField::IncSum, EComplexStatField::IncAve);
			}

			FComplexStatUtils::DiviveStatArray( AggregatedNonStackStatsHistory, NumFrames, EComplexStatField::IncSum, EComplexStatField::IncAve );
	
			// Sort groups into a reliable order, based on the order they were activated.
			// More recently activated stats appear higher on screen than older stats.
			TArray<const FInternalGroup*> SortedInternalGroups;
			SortedInternalGroups.Reserve(EnabledGroups.Num());
			for (auto const& [Name, InternalGroup] : EnabledGroups)
			{
				SortedInternalGroups.Add(&InternalGroup);
			}

			SortedInternalGroups.Sort([](const FInternalGroup& LHS, const FInternalGroup& RHS)
			{
				return LHS.SortOrder >= RHS.SortOrder;
			});

			// Iterate through all enabled groups.
			for (const FInternalGroup* InternalGroupPtr : SortedInternalGroups)
			{
				const FInternalGroup& InternalGroup = *InternalGroupPtr;
				const FName& GroupName = InternalGroup.GroupName;

				// Create a new hud group.
				ToGame->ActiveStatGroups.Add(new FActiveStatGroupInfo());
				FActiveStatGroupInfo& HudGroup = ToGame->ActiveStatGroups.Last();

				ToGame->GroupNames.Add( GroupName );
				ToGame->GroupDescriptions.Add( InternalGroup.GroupDescription );
				HudGroup.ThreadBudgetMap = InternalGroup.ThreadBudgetMap;
				HudGroup.BudgetIgnoreStats = InternalGroup.BudgetIgnoreStats;

				if (Params.bSlowMode)
				{
					// Linearize stack stats for easier rendering.
					LinearizeSlowStackForItems( AggregatedHierarchyHistory, HudGroup.HierAggregate, HudGroup.Indentation, 0 );
				}
				else
				{
				if( InternalGroup.DisplayMode & EStatDisplayMode::Hierarchical )
				{
					// Linearize stack stats for easier rendering.
					LinearizeStackForItems( AggregatedHierarchyHistory, InternalGroup.EnabledItems, HudGroup.HierAggregate, HudGroup.Indentation, 0 );
				}
				
				if( InternalGroup.DisplayMode & EStatDisplayMode::Flat )
				{
					// Copy flat stats
					for( int32 Index = 0; Index < AggregatedFlatHistory.Num(); ++Index )
					{
						const FComplexStatMessage& AggregatedStatMessage = AggregatedFlatHistory[Index];
						const bool bIsNonStackStat = !AggregatedStatMessage.NameAndInfo.GetFlag(EStatMetaFlags::IsPackedCCAndDuration);
						const bool bToBeAdded = InternalGroup.EnabledItems.Contains( AggregatedStatMessage.NameAndInfo.GetRawName() );
						if( bToBeAdded )
						{
							HudGroup.FlatAggregate.Add( AggregatedStatMessage );
						}
					}

					for(TMap<FName, TArray<FComplexStatMessage>>::TConstIterator It(AggregatedFlatHistoryThreadBreakdown); It; ++It)
					{
						const TArray<FComplexStatMessage>& SrcArray = It.Value();
						
						for(const FComplexStatMessage& AggregatedStatMessage : SrcArray)
						{
							const bool bToBeAdded = InternalGroup.EnabledItems.Contains(AggregatedStatMessage.NameAndInfo.GetRawName());
							if(bToBeAdded)
							{
								TArray<FComplexStatMessage>& DestArray = HudGroup.FlatAggregateThreadBreakdown.FindOrAdd(It.Key());	
								DestArray.Add( AggregatedStatMessage );
							}
						}
					}
				}

				// Copy non-stack stats assigned to memory and counter groups.
				for( int32 Index = 0; Index < AggregatedNonStackStatsHistory.Num(); ++Index )
				{
					const FComplexStatMessage& AggregatedStatMessage = AggregatedNonStackStatsHistory[Index];
					const bool bIsMemory = AggregatedStatMessage.NameAndInfo.GetFlag( EStatMetaFlags::IsMemory );
					const bool bIsGpu = AggregatedStatMessage.NameAndInfo.GetFlag( EStatMetaFlags::IsGPU );

					TArray<FComplexStatMessage>& Dest = bIsGpu ? HudGroup.GpuStatsAggregate : (bIsMemory ? HudGroup.MemoryAggregate : HudGroup.CountersAggregate); 

					const bool bToBeAdded = InternalGroup.EnabledItems.Contains( AggregatedStatMessage.NameAndInfo.GetRawName() );
					if( bToBeAdded )
					{
						Dest.Add(AggregatedStatMessage);
					}
				}
			}

				// Replace thread encoded id with the thread name.
				for (auto& HierIt : HudGroup.HierAggregate)
				{
					FComplexStatMessage& StatMessage = HierIt;
					const FString StatDescription = StatMessage.NameAndInfo.GetDescription();
					if (StatDescription.Contains( FStatConstants::ThreadNameMarker ))
					{
						StatMessage.NameAndInfo.SetRawName( StatMessage.NameAndInfo.GetShortName() );
					}
				}
			}

			for (auto It = Stats.MemoryPoolToCapacityLongName.CreateConstIterator(); It; ++It)
			{
				const FName LongName = It.Value();
				// dig out the abbreviation
				{
					const FString LongNameStr = LongName.ToString();
					const int32 Open = LongNameStr.Find("[");
					const int32 Close = LongNameStr.Find("]");
					if (Open >= 0 && Close >= 0 && Open + 1 < Close)
					{
						FString Abbrev = LongNameStr.Mid(Open + 1, Close - Open - 1);
						ToGame->PoolAbbreviation.Add(It.Key(), Abbrev);
					}
				}
				// see if we have a capacity
				FStatMessage const* Result = Stats.NotClearedEveryFrame.Find(LongName);
				if (Result && Result->NameAndInfo.GetFlag(EStatMetaFlags::IsMemory))
				{
					const int64 Capacity = Result->GetValue_int64();
					if (Capacity > 0)
					{
						ToGame->PoolCapacity.Add(It.Key(), Capacity);
					}
				}
			}

			{
				const TIndirectArray<FActiveStatGroupInfo>& ActiveGroups = ToGame->ActiveStatGroups;
				for(const FActiveStatGroupInfo& GroupInfo : ActiveGroups)
				{
					for(const FComplexStatMessage& StatMessage : GroupInfo.FlatAggregate)
					{
						const FName StatName = StatMessage.GetShortName();
						ToGame->NameToStatMap.Add(StatName, &StatMessage);
					}
				}
			}

			DECLARE_CYCLE_STAT(TEXT("FSimpleDelegateGraphTask.StatsHierToGame"),
				STAT_FSimpleDelegateGraphTask_StatsHierToGame,
				STATGROUP_TaskGraphTasks);

			FSimpleDelegateGraphTask::CreateAndDispatchWhenReady
			(
				FSimpleDelegateGraphTask::FDelegate::CreateRaw(&FLatestGameThreadStatsData::Get(), &FLatestGameThreadStatsData::NewData, ToGame),
				GET_STATID(STAT_FSimpleDelegateGraphTask_StatsHierToGame), nullptr, ENamedThreads::GameThread
			);
		}	
	}

	void RemoveFramesOutOfHistory( int64 TargetFrame )
	{
		LatestFrame = TargetFrame;
		for (auto It = History.CreateIterator(); It; ++It)
		{
			if (int32(LatestFrame - It.Key()) >= Params.MaxHistoryFrames.Get())
			{
				It.RemoveCurrent();
			}
		}
		check(History.Num() <= Params.MaxHistoryFrames.Get());
	}

	void GetStatsForNames(TSet<FName>& out_EnabledItems, const TArray<FName>& ShortNames)
	{
		for (const FName& ShortName : ShortNames)
		{
			out_EnabledItems.Add(ShortName);
			if (FStatMessage const* LongName = Stats.ShortNameToLongName.Find(ShortName))
			{
				out_EnabledItems.Add(LongName->NameAndInfo.GetRawName()); // long name
			}
		}
	}

	void GetStatsForGroup( TSet<FName>& out_EnabledItems, const FName GroupName )
	{
		out_EnabledItems.Reset();
	
		TArray<FName> GroupItems;
		Stats.Groups.MultiFind( GroupName, GroupItems );

		GetStatsForNames(out_EnabledItems, GroupItems);

		out_EnabledItems.Add(NAME_Self);
		out_EnabledItems.Add(NAME_OtherChildren);
	}

	static FHUDGroupManager& Get(FStatsThreadState const& Stats)
	{
		static FHUDGroupManager Singleton(Stats);
		return Singleton;
	}
};

bool FGroupFilter::IsRoot(const FName& MessageName) const
{
	bool bIsRoot = false;
	if (bool* bResult = HudGroupManager->RootFilterCache.Find(MessageName))
	{
		bIsRoot = *bResult;
	}
	else
	{
		bIsRoot = MessageName.ToString().Contains(RootFilter);
		HudGroupManager->RootFilterCache.Add(MessageName, bIsRoot);
	}

	return bIsRoot;
}


/*-----------------------------------------------------------------------------
	Dump...
-----------------------------------------------------------------------------*/

static int32 GMaxDepth = MAX_int32;
static FString GNameFilter;
static FString GLeafFilter;
static FDelegateHandle GDumpFrameDelegateHandle;
static FDelegateHandle GDumpCPUDelegateHandle;

static void DumpFrame(int64 Frame)
{
	FStatsThreadState& Stats = FStatsThreadState::GetLocalState();
	int64 Latest = Stats.GetLatestValidFrame();
	check(Latest > 0);
	DumpHistoryFrame(Stats, Latest, DumpCull, GMaxDepth, *GNameFilter);
	Stats.NewFrameDelegate.Remove(GDumpFrameDelegateHandle);
	StatsPrimaryEnableSubtract();
}

static void DumpCPU(int64 Frame)
{
	FStatsThreadState& Stats = FStatsThreadState::GetLocalState();
	int64 Latest = Stats.GetLatestValidFrame();
	check(Latest > 0);
	DumpCPUSummary(Stats, Latest);
	Stats.NewFrameDelegate.Remove(GDumpCPUDelegateHandle);
	StatsPrimaryEnableSubtract();
}

static struct FDumpMultiple* DumpMultiple = NULL;

struct FDumpMultiple
{
	FStatsThreadState& Stats;
	bool bAverage;
	bool bSum;
	int32 NumFrames;
	int32 NumFramesToGo;
	FRawStatStackNode* Stack;
	FDelegateHandle NewFrameDelegateHandle;

	FDumpMultiple()
		: Stats(FStatsThreadState::GetLocalState())
		, bAverage(true)
		, bSum(false)
		, NumFrames(0)
		, NumFramesToGo(0)
		, Stack(NULL)
	{
		StatsPrimaryEnableAdd();
		NewFrameDelegateHandle = Stats.NewFrameDelegate.AddRaw(this, &FDumpMultiple::NewFrame);
	}

	~FDumpMultiple()
	{
		if (Stack && NumFrames)
		{
			if (bAverage)
			{
				if (NumFrames > 1)
				{
					Stack->Divide(NumFrames);
				}
				UE_LOGF(LogStats, Log, "------------------ %d frames, average ---------------", NumFrames );
			}
			else if (bSum)
			{
				UE_LOGF(LogStats, Log, "------------------ %d frames, sum ---------------", NumFrames );
			}
			else
			{
				UE_LOGF(LogStats, Log, "------------------ %d frames, max ---------------", NumFrames );
			}
			Stack->AddNameHierarchy();
			Stack->AddSelf();
			if (DumpCull != 0.0f)
			{
				Stack->CullByCycles( int64( DumpCull / FPlatformTime::ToMilliseconds( 1 ) ) );
			}
			if (!GNameFilter.IsEmpty() && !GLeafFilter.IsEmpty())
			{
				UE_LOGF(LogStats, Log, "You can't have both a root and a leaf filter (though this wouldn't be hard to add).");
			}
			else if (!GLeafFilter.IsEmpty())
			{
				Stack->DebugPrintLeafFilter(*GLeafFilter);
			}
			else
			{
				Stack->DebugPrint(*GNameFilter, GMaxDepth);
			}
			delete Stack;
			Stack = NULL;
		}
		Stats.NewFrameDelegate.Remove(NewFrameDelegateHandle);
		StatsPrimaryEnableSubtract();
		DumpMultiple = NULL;
	}

	void NewFrame(int64 TargetFrame)
	{
		if (!Stack)
		{
			Stack = new FRawStatStackNode();
			Stats.UncondenseStackStats(TargetFrame, *Stack);
		}
		else
		{
			FRawStatStackNode FrameStack;
			Stats.UncondenseStackStats(TargetFrame, FrameStack);
			if (bAverage || bSum)
			{
				Stack->MergeAdd(FrameStack);
			}
			else
			{
				Stack->MergeMax(FrameStack);
			}
		}
		NumFrames++;
		if (NumFrames >= NumFramesToGo)
		{
			delete this;
		}
	}
};

static struct FDumpSpam* DumpSpam = nullptr;

struct FDumpSpam
{
	FStatsThreadState& Stats;
	TMap<FName, int32> Counts;
	int32 TotalCount;
	int32 NumPackets;
	FDelegateHandle NewRawStatPacketDelegateHandle;

	FDumpSpam()
		: Stats(FStatsThreadState::GetLocalState())
		, TotalCount(0)
		, NumPackets(0)
	{
		FThreadStats::EnableRawStats();
		StatsPrimaryEnableAdd();
		NewRawStatPacketDelegateHandle = Stats.NewRawStatPacket.AddRaw(this, &FDumpSpam::NewFrame);
	}

	~FDumpSpam()
	{
		FThreadStats::DisableRawStats();
		StatsPrimaryEnableSubtract();
		UE_LOGF(LogStats, Log, "------------------ %d packets, %d total messages ---------------", NumPackets, TotalCount);

		Counts.ValueSort(TGreater<int32>());

		for (auto& Pair : Counts)
		{
			UE_LOGF(LogStats, Log, "%10d	  %ls", Pair.Value, *Pair.Key.ToString());
		}

		Stats.NewRawStatPacket.Remove(NewRawStatPacketDelegateHandle);
		DumpSpam = NULL;
	}

	void NewFrame(const FStatPacket* Packet)
	{
		NumPackets++;
		TotalCount += Packet->StatMessages.Num();
		for( const FStatMessage& Message : Packet->StatMessages )
		{
			FName Name = Message.NameAndInfo.GetRawName();
			int32* Existing = Counts.Find(Name);
			if (Existing)
			{
				(*Existing)++;
			}
			else
			{
				Counts.Add(Name, 1);
			}
		}
	}
};

/** Prints stats help to the specified output device. This is queued to be executed on the game thread. */
static void PrintStatsHelpToOutputDevice( FOutputDevice& Ar )
{
	Ar.Log( TEXT("Empty stat command!"));
	Ar.Log( TEXT("Here is the brief list of stats console commands"));
	Ar.Log( TEXT("stat dumpframe [-ms=5.0] [-root=empty] [-depth=maxint] - dumps a frame of stats"));
	Ar.Log( TEXT("	stat dumpframe -ms=.001 -root=initviews"));
	Ar.Log( TEXT("	stat dumpframe -ms=.001 -root=shadow"));

	Ar.Log( TEXT("stat dumpave|dumpmax|dumpsum  [-start | -stop | -num=30] [-ms=5.0] [-depth=maxint] - aggregate stats over multiple frames"));
	Ar.Log( TEXT("stat dumphitches - toggles dumping hitches"));
	Ar.Log( TEXT("stat dumpevents [-ms=0.2] [-all] - dumps events history for slow events, -all adds other threads besides game and render"));
	Ar.Log( TEXT("stat dumpnonframe - dumps non-frame stats, usually memory stats"));
	Ar.Log( TEXT("stat dumpcpu - dumps cpu stats"));

	Ar.Log( TEXT("stat groupname[+] - toggles displaying stats group, + enables hierarchical display"));
	Ar.Log( TEXT("stat hier -group=groupname [-sortby=name] [-maxhistoryframes=60] [-reset] [-maxdepth=4]"));
	Ar.Log( TEXT("	- groupname is a stat group like initviews or statsystem"));
	Ar.Log( TEXT("	- sortby can be name (by stat FName), callcount (by number of calls, only for scoped cycle counters), num(by total inclusive time)"));
	Ar.Log( TEXT("	- maxhistoryframes (default 60, number of frames used to generate the stats displayed on the hud)"));
	Ar.Log( TEXT("	- reset (reset the accumulated history)"));
	Ar.Log( TEXT("	- maxdepth (default 4, maximum depth for the hierarchy)"));
	Ar.Log( TEXT("stat none - disables drawing all stats groups"));

	Ar.Log( TEXT("stat group list|listall|enable name|disable name|none|all|default - manages stats groups"));

#if WITH_ENGINE
	// Engine @see FStatCmdEngine
	Ar.Log( TEXT( "stat display -font=small[tiny]" ) );
	Ar.Log( TEXT( "	Changes stats rendering display options" ) );
#endif // WITH_ENGINE

#if UE_ENABLE_STATS_FILE_DEPRECATED_IN_5_8
	Ar.Log( TEXT("stat startfile - starts dumping a capture"));
	Ar.Log( TEXT("stat stopfile - stops dumping a capture (regular, raw, memory)"));
	Ar.Log( TEXT("stat startfileraw - starts dumping a raw capture"));
	Ar.Log( TEXT("stat testfile - loads the last saved capture and dumps first, middle and last frame"));
#endif // UE_ENABLE_STATS_FILE_DEPRECATED_IN_5_8

	Ar.Log( TEXT("stat toggledebug - toggles tracking the most memory expensive stats"));

	Ar.Log( TEXT( "stat slow [-ms=1.0] [-depth=4] - toggles displaying the game and render thread stats" ) );

	Ar.Log( TEXT("stat namedmarker #markername# - adds a custom marker to the stats stream"));
}

#endif

/** bStatCommand indicates whether we are coming from a stat command or a budget command */
static void StatCmd(FString InCmd, bool bStatCommand, FOutputDevice* Ar /*= nullptr*/)
{
	const TCHAR* Cmd = *InCmd;
	if(bStatCommand)
	{
#if STATS
		FStatsThreadState& Stats = FStatsThreadState::GetLocalState();
		DumpCull = 1.0f;
		GMaxDepth = MAX_int32;
		GNameFilter.Empty();
		GLeafFilter.Empty();

		FParse::Value(Cmd, TEXT("ROOT="), GNameFilter);
		FParse::Value(Cmd, TEXT("LEAF="), GLeafFilter);
		FParse::Value(Cmd, TEXT("MS="), DumpCull);
		FParse::Value(Cmd, TEXT("DEPTH="), GMaxDepth);
		if (FParse::Command(&Cmd, TEXT("DUMPFRAME")))
		{
			StatsPrimaryEnableAdd();
			GDumpFrameDelegateHandle = Stats.NewFrameDelegate.AddStatic(&DumpFrame);
		}
		else if (FParse::Command(&Cmd, TEXT("DUMPNONFRAME")))
		{
			FString MaybeGroup;
			FParse::Token(Cmd, MaybeGroup, false);

			DumpNonFrame(Stats, MaybeGroup.Len() == 0 ? NAME_None : FName(*(FString("STATGROUP_") + MaybeGroup)));
		}
		else if (FParse::Command(&Cmd, TEXT("DUMPCPU")))
		{
			StatsPrimaryEnableAdd();
			GDumpCPUDelegateHandle = Stats.NewFrameDelegate.AddStatic(&DumpCPU);
		}
		else if (FParse::Command(&Cmd, TEXT("STOP")))
		{
			delete DumpMultiple;
		}
		else if (FParse::Command(&Cmd, TEXT("DUMPAVE")))
		{
			bool bIsStart = FString(Cmd).Find(TEXT("-start")) != INDEX_NONE;
			bool bIsStop = FString(Cmd).Find(TEXT("-stop")) != INDEX_NONE;
			delete DumpMultiple;
			if (!bIsStop)
			{
				DumpMultiple = new FDumpMultiple();
				DumpMultiple->NumFramesToGo = bIsStart ? MAX_int32 : 30;
				FParse::Value(Cmd, TEXT("NUM="), DumpMultiple->NumFramesToGo);
				DumpMultiple->bAverage = true;
				DumpMultiple->bSum = false;
			}
		}
		else if (FParse::Command(&Cmd, TEXT("DUMPMAX")))
		{
			bool bIsStart = FString(Cmd).Find(TEXT("-start")) != INDEX_NONE;
			bool bIsStop = FString(Cmd).Find(TEXT("-stop")) != INDEX_NONE;
			delete DumpMultiple;
			if (!bIsStop)
			{
				DumpMultiple = new FDumpMultiple();
				DumpMultiple->NumFramesToGo = bIsStart ? MAX_int32 : 30;
				FParse::Value(Cmd, TEXT("NUM="), DumpMultiple->NumFramesToGo);
				DumpMultiple->bAverage = false;
				DumpMultiple->bSum = false;
			}
		}
		else if (FParse::Command(&Cmd, TEXT("DUMPSUM")))
		{
			bool bIsStart = FString(Cmd).Find(TEXT("-start")) != INDEX_NONE;
			bool bIsStop = FString(Cmd).Find(TEXT("-stop")) != INDEX_NONE;
			delete DumpMultiple;
			if (!bIsStop)
			{
				DumpMultiple = new FDumpMultiple();
				DumpMultiple->NumFramesToGo = bIsStart ? MAX_int32 : 30;
				FParse::Value(Cmd, TEXT("NUM="), DumpMultiple->NumFramesToGo);
				DumpMultiple->bAverage = false;
				DumpMultiple->bSum = true;
			}
		}
		else if (FParse::Command(&Cmd, TEXT("DUMPSPAM")))
		{
			bool bIsStart = FString(Cmd).Find(TEXT("-start")) != INDEX_NONE;
			bool bIsStop = FString(Cmd).Find(TEXT("-stop")) != INDEX_NONE;
			delete DumpSpam;
			if (!bIsStop)
			{
				DumpSpam = new FDumpSpam();
			}
		}
		else if (FParse::Command(&Cmd, TEXT("DUMPHITCHES")))
		{
			static bool bToggle = false;
			static FDelegateHandle DumpHitchDelegateHandle;
			
			bool bIsStart = FString(Cmd).Find(TEXT("-start")) != INDEX_NONE;
			bool bIsStop = FString(Cmd).Find(TEXT("-stop")) != INDEX_NONE;

			if (bIsStart && bToggle)
				return;

			if (bIsStop && !bToggle)
				return;

			bToggle = !bToggle;
			if (bToggle)
			{
				StatsPrimaryEnableAdd();
				HitchIndex = 0;
				TotalHitchTime = 0.0;
				DumpHitchDelegateHandle = Stats.NewFrameDelegate.AddStatic(&DumpHitch);
			}
			else
			{
				StatsPrimaryEnableSubtract();
				Stats.NewFrameDelegate.Remove(DumpHitchDelegateHandle);
				UE_LOGF(LogStats, Log, "**************************** %d hitches	%8.0fms total hitch time", HitchIndex, TotalHitchTime);
			}
			if (Ar)
			{
				Ar->Logf(TEXT("dumphitches set to %d"), bToggle);
			}
		}
		else if (FParse::Command(&Cmd, TEXT("DumpEvents")))
		{
			float DumpEventsCullMS = 0.1f;
			FParse::Value(Cmd, TEXT("MS="), DumpEventsCullMS);
			const bool bDisplayAllThreads = FParse::Param(Cmd, TEXT("all"));

			StatsPrimaryEnableAdd();
			DumpEventsDelegateHandle = Stats.NewFrameDelegate.AddStatic(&DumpEventsOnce, DumpEventsCullMS, bDisplayAllThreads);
		}
#if UE_ENABLE_STATS_FILE_DEPRECATED_IN_5_8
		else if (FParse::Command(&Cmd, TEXT("STARTFILE")))
		{
			FString Filename;
			FParse::Token(Cmd, Filename, false);
			FCommandStatsFile::Get().Start(Filename);
		}
		else if (FParse::Command(&Cmd, TEXT("StartFileRaw")))
		{
			FThreadStats::EnableRawStats();
			FString Filename;
			FParse::Token(Cmd, Filename, false);
			FCommandStatsFile::Get().StartRaw(Filename);
		}
		else if (FParse::Command(&Cmd, TEXT("STOPFILE"))
			|| FParse::Command(&Cmd, TEXT("StopFileRaw")))
		{
			// Stop writing to a file.
			FCommandStatsFile::Get().Stop();
			FThreadStats::DisableRawStats();

			Stats.ResetStatsForRawStats();

			// stopfile command happens when some threads shutdown, and depending on order of shutdown operations, the taskgraph may
			// shutdown before stopfile command is executed
			if (FTaskGraphInterface::IsRunning())
			{
				// Disable displaying the raw stats memory overhead.
				FSimpleDelegateGraphTask::CreateAndDispatchWhenReady
					(
					 FSimpleDelegateGraphTask::FDelegate::CreateRaw(&FLatestGameThreadStatsData::Get(), &FLatestGameThreadStatsData::NewData, (FGameThreadStatsData*)nullptr),
					 TStatId(), nullptr, ENamedThreads::GameThread
					 );
			}
			else
			{
				FLatestGameThreadStatsData::Get().NewData(nullptr);
			}
		}
		else if (FParse::Command(&Cmd, TEXT("TESTFILE")))
		{
			FCommandStatsFile::Get().TestLastSaved();
		}
#endif // UE_ENABLE_STATS_FILE_DEPRECATED_IN_5_8
		else if (FParse::Command(&Cmd, TEXT("testdisable")))
		{
			FThreadStats::PrimaryDisableForever();
		}
		else if (FParse::Command(&Cmd, TEXT("none")))
		{
			FStatParams Params;
			FHUDGroupManager::Get(Stats).HandleCommand(Params, false);
		}
		else if (FParse::Command(&Cmd, TEXT("group")))
		{
			IStatGroupEnableManager::Get().StatGroupEnableManagerCommand(Cmd);
		}
		else if (FParse::Command(&Cmd, TEXT("toggledebug")))
		{
			FStatsThreadState::GetLocalState().ToggleFindMemoryExtensiveStats();
		}
		else if (FParse::Command(&Cmd, TEXT("namedmarker")))
		{
			FString MarkerName;
			FParse::Token(Cmd, MarkerName, false);

			struct FLocal
			{
				static void OnGameThread(FString InMarkerName)
				{
					const FName NAME_Marker = FName(*InMarkerName);
					STAT_ADD_CUSTOMMESSAGE_NAME(STAT_NamedMarker, NAME_Marker);
					UE_LOGF(LogStats, Log, "Added from console STAT_NamedMarker: %ls", *InMarkerName);
				}
			};

			if (!MarkerName.IsEmpty())
			{
				// This will be executed on the game thread.
				FSimpleDelegateGraphTask::CreateAndDispatchWhenReady
				(
					FSimpleDelegateGraphTask::FDelegate::CreateStatic(&FLocal::OnGameThread, MarkerName),
					TStatId(), nullptr, ENamedThreads::GameThread
				);
			}
		}
		// @see FStatHierParams
		else if (FParse::Command(&Cmd, TEXT("hier")))
		{
			FStatParams Params(Cmd);
			FHUDGroupManager::Get(Stats).HandleCommand(Params, true);
		}
		else if (FParse::Command(&Cmd, TEXT("slow")))
		{
			FStatSlowParams Params(Cmd);
			FHUDGroupManager::Get(Stats).HandleCommand(Params, true);
		}
		else if (FParse::Command(&Cmd, TEXT("gpu")))
		{
			// Macro command to toggle all the stat groups that contain "STATCAT_GPU" stats.
			if (FCoreDelegates::StatCheckEnabled.IsBound())
			{
				// Gather the list of relevant stat groups
				TSet<FName> GpuStatGroups;
				for (auto const& [GroupName, StatName] : Stats.Groups)
				{
					FName Category = Stats.ShortNameToLongName.FindChecked(StatName).NameAndInfo.GetGroupCategory();
					if (Category == FStatNameAndInfo::GpuStatCategory)
					{
						GpuStatGroups.Add(GroupName);
					}
				}

				if (GpuStatGroups.Num() > 0)
				{
					// Check if any of these groups are already enabled.
					bool bAllEnabled = true;
					for (FName Group : GpuStatGroups)
					{
						FString GroupName = Group.ToString();
						GroupName.RemoveFromStart(TEXT("STATGROUP_"));

						bool bEnabled = false, bOtherEnabled = false;
						FCoreDelegates::StatCheckEnabled.Broadcast(*GroupName, bEnabled, bOtherEnabled);

						if (!bEnabled)
						{
							bAllEnabled = false;
							break;
						}
					}

					bool bShouldBeEnabled = !bAllEnabled;

					// Toggle any stat groups which don't match the desired state.
					for (FName Group : GpuStatGroups)
					{
						FString GroupName = Group.ToString();
						GroupName.RemoveFromStart(TEXT("STATGROUP_"));

						bool bEnabled = false, bOtherEnabled = false;
						FCoreDelegates::StatCheckEnabled.Broadcast(*GroupName, bEnabled, bOtherEnabled);

						if (bEnabled != bShouldBeEnabled)
						{
							FStatParams Params(Cmd);
							Params.Group.Set(*GroupName);

							FHUDGroupManager::Get(Stats).HandleCommand(Params, false);
						}
					}
				}
			}
		}
		else
#endif
		{
			FString MaybeGroup;
			FParse::Token(Cmd, MaybeGroup, false);

			if (MaybeGroup.Len() > 0)
			{
				// If there is + at the end of the group name switch into hierarchical view mode.
				const int32 PlusPos = MaybeGroup.Len() - 1;
				const bool bHierarchy = MaybeGroup[MaybeGroup.Len() - 1] == TEXT('+');
				if (bHierarchy)
				{
					MaybeGroup.RemoveAt(PlusPos, EAllowShrinking::No);
				}

				const FName MaybeGroupFName = FName(*MaybeGroup);
#if STATS
				// Try to parse.
				FStatParams Params(Cmd);
				Params.Group.Set(MaybeGroupFName);
				FHUDGroupManager::Get(Stats).HandleCommand(Params, bHierarchy);

				GRenderStats = !FParse::Command(&Cmd, TEXT("-nodisplay"));	//allow us to hide the rendering of stats
#else
				// If stats aren't enabled, broadcast so engine stats can still be triggered
				bool bCurrentEnabled, bOthersEnabled;
				HandleToggleCommandBroadcast(MaybeGroupFName, bCurrentEnabled, bOthersEnabled);
#endif
			}
			else
			{
				// Display a help. Handled by DirectStatsCommand.
			}
		}
	}
	else
	{
		FString MaybeBudget;
		FParse::Token(Cmd, MaybeBudget, false);

		if (MaybeBudget.Len() > 0)
		{
#if STATS
			// Try to parse.
			FStatParams Params(Cmd);
			Params.BudgetSection = *MaybeBudget;
			Params.Group.Set(FName("Budget"));
			FStatsThreadState& Stats = FStatsThreadState::GetLocalState();
			FHUDGroupManager::Get(Stats).HandleCommand(Params, false);
#endif
		}
	}
}

/** Exec used to execute core stats commands on the stats thread. */
static class FStatCmdCore : private FSelfRegisteringExec
{
protected:
	/** Console commands, see embeded usage statement **/
	virtual bool Exec_Runtime( UWorld*, const TCHAR* Cmd, FOutputDevice& Ar ) override
	{
		// Block the thread as this affects external stat states now
		return UE::Stats::DirectStatsCommand(Cmd,true,&Ar);
	}
}
StatCmdCoreExec;

namespace UE::Stats
{
	bool DirectStatsCommand(const TCHAR* Cmd, bool bBlockForCompletion /*= false*/, FOutputDevice* Ar /*= nullptr*/)
	{
		bool bResult = false;
		const bool bStatCommand = FParse::Command(&Cmd, TEXT("stat"));
		const bool bBudgetCommand = FParse::Command(&Cmd, TEXT("budget"));

		if (bStatCommand || bBudgetCommand)
		{
			FString AddArgs;
			const TCHAR* TempCmd = Cmd;

			FString ArgNoWhitespaces = FDefaultValueHelper::RemoveWhitespaces(TempCmd);
			const bool bIsEmpty = ArgNoWhitespaces.IsEmpty();
#if STATS
			bResult = true;

			if (bStatCommand)
			{
				if (bIsEmpty && Ar)
				{
					PrintStatsHelpToOutputDevice(*Ar);
				}
#if UE_ENABLE_STATS_FILE_DEPRECATED_IN_5_8
				else if (FParse::Command(&TempCmd, TEXT("STARTFILE")))
				{
					FString Filename;
					AddArgs += TEXT(" ");
					if (FParse::Line(&TempCmd, Filename, true))
					{
						AddArgs += Filename;
					}
					else
					{
						AddArgs += CreateProfileFilename(FStatConstants::StatsFileExtension, true);
					}
				}
				else if (FParse::Command(&TempCmd, TEXT("StartFileRaw")))
				{
					AddArgs += TEXT(" ");
					AddArgs += CreateProfileFilename(FStatConstants::StatsFileRawExtension, true);
				}
#endif // UE_ENABLE_STATS_FILE_DEPRECATED_IN_5_8
				else if (FParse::Command(&TempCmd, TEXT("DUMPFRAME")))
				{
				}
				else if (FParse::Command(&TempCmd, TEXT("DUMPNONFRAME")))
				{
				}
				else if (FParse::Command(&TempCmd, TEXT("DUMPCPU")))
				{
				}
				else if (FParse::Command(&TempCmd, TEXT("STOP")))
				{
				}
				else if (FParse::Command(&TempCmd, TEXT("DUMPAVE")))
				{
				}
				else if (FParse::Command(&TempCmd, TEXT("DUMPMAX")))
				{
				}
				else if (FParse::Command(&TempCmd, TEXT("DUMPSUM")))
				{
				}
				else if (FParse::Command(&TempCmd, TEXT("DUMPSPAM")))
				{
				}
				else if (FParse::Command(&TempCmd, TEXT("DUMPHITCHES")))
				{
				}
				else if (FParse::Command(&TempCmd, TEXT("DumpEvents")))
				{
				}
#if UE_ENABLE_STATS_FILE_DEPRECATED_IN_5_8
				else if (FParse::Command(&TempCmd, TEXT("STOPFILE")))
				{
				}
				else if (FParse::Command(&TempCmd, TEXT("TESTFILE")))
				{
				}
#endif // UE_ENABLE_STATS_FILE_DEPRECATED_IN_5_8
				else if (FParse::Command(&TempCmd, TEXT("testdisable")))
				{
				}
				else if (FParse::Command(&TempCmd, TEXT("none")))
				{
				}
				else if (FParse::Command(&TempCmd, TEXT("group")))
				{
				}
				else if (FParse::Command(&TempCmd, TEXT("hier")))
				{
				}
				else if (FParse::Command(&TempCmd, TEXT("net")))
				{
				}
				else if (FParse::Command(&TempCmd, TEXT("toggledebug")))
				{
				}
				else if (FParse::Command(&TempCmd, TEXT("memoryprofiler")))
				{
				}
				else if (FParse::Command(&TempCmd, TEXT("slow")))
				{
				}
				else if (FParse::Command(&TempCmd, TEXT("gpu")))
				{
				}
				else if (FParse::Command(&TempCmd, TEXT("namedmarker")))
				{
				}
				else
				{
					bResult = false;

					FString MaybeGroup;
					if (FParse::Token(TempCmd, MaybeGroup, false) && MaybeGroup.Len() > 0)
					{
						// If there is + at the end of the group name, remove it
						const int32 PlusPos = MaybeGroup.Len() - 1;
						const bool bHierarchy = MaybeGroup[MaybeGroup.Len() - 1] == TEXT('+');
						if (bHierarchy)
						{
							MaybeGroup.RemoveAt(PlusPos, EAllowShrinking::No);
						}

						const FName MaybeGroupFName = FName(*(FString("STATGROUP_") + MaybeGroup));
						bResult = FStatGroupGameThreadNotifier::Get().StatGroupNames.Contains(MaybeGroupFName);
					}
				}
			}
			else
			{
				FString BudgetSection;
				const TCHAR* TmpCmd = Cmd;
				if (FParse::Token(TmpCmd, BudgetSection, false) && BudgetSection.Len() > 0)
				{
					FScopeLock BudgetINILock(&BudgetStatMapCS);   //Make sure stats thread isn't currently reading from this data
					{
						FBudgetData& BudgetData = BudgetStatMapping.FindOrAdd(*BudgetSection);
						BudgetData = FBudgetData();
						GConfig->GetArray(*BudgetSection, TEXT("Stats"), BudgetData.Stats, GEngineIni);

						TArray<FString> Lines;
						GConfig->GetSection(*BudgetSection, Lines, GEngineIni);
						for (const FString& Line : Lines)
						{
							if (!Line.Contains(TEXT("+Stats=")))	//ignore stats array
							{
								FString ThreadName;
								Line.Split(FString("="), &ThreadName, nullptr);
								float Budget = -1.f;
								if (GConfig->GetFloat(*BudgetSection, *ThreadName, Budget, GEngineIni))
								{
									BudgetData.ThreadBudgetMap.FindOrAdd(FName(*ThreadName)) = Budget;
								}
							}
						}

						BudgetData.Process();
					}
				}
			}

#endif

			check(IsInGameThread());
			if (!bIsEmpty)
			{
				const FString FullCmd = FString(Cmd) + AddArgs;
#if STATS
				// make sure these are initialized on the game thread
				FLatestGameThreadStatsData::Get();
				FStatGroupGameThreadNotifier::Get();

				DECLARE_CYCLE_STAT(TEXT("FSimpleDelegateGraphTask.StatCmd"),
					STAT_FSimpleDelegateGraphTask_StatCmd,
					STATGROUP_TaskGraphTasks);

				UE::Tasks::FTask Task = GStatsPipe.Launch(UE_SOURCE_LOCATION, [FullCmd, bStatCommand, Ar] { StatCmd(FullCmd, bStatCommand, Ar); });
				if (bBlockForCompletion)
				{
					Task.Wait();
					GLog->FlushThreadedLogs();
				}
#else // STATS
				// If stats aren't enabled, broadcast so engine stats can still be triggered
				StatCmd(FullCmd, bStatCommand, Ar);
#endif // STATS
			}
		}
		return bResult;
	}
} // namespace UE::Stats

#if STATS

static void GetPermanentStats_StatsThread(TArray<FStatMessage>* OutStats)
{
	DECLARE_CYCLE_STAT(TEXT("FSimpleDelegateGraphTask.GetPermanentStatsString_StatsThread"),
		STAT_FSimpleDelegateGraphTask_GetPermanentStatsString_StatsThread,
		STATGROUP_TaskGraphTasks);
	SCOPE_CYCLE_COUNTER(STAT_FSimpleDelegateGraphTask_GetPermanentStatsString_StatsThread);

	FStatsThreadState& StatsData = FStatsThreadState::GetLocalState();
	TArray<FStatMessage>& Stats = *OutStats;
	for (auto It = StatsData.NotClearedEveryFrame.CreateConstIterator(); It; ++It)
	{
		Stats.Add(It.Value());
	}
	Stats.Sort(FGroupSort());
}

void GetPermanentStats(TArray<FStatMessage>& OutStats)
{
	GStatsPipe
		.Launch(UE_SOURCE_LOCATION, [&OutStats] { GetPermanentStats_StatsThread(&OutStats); })
		.Wait();
}

#endif
