// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stats/StatsData.h"
#include "Templates/Greater.h"
#include "Misc/CoreStats.h"
#if STATS

#include "Async/TaskGraphInterfaces.h"
#include "Containers/LockFreeFixedSizeAllocator.h"
#include "Containers/StringView.h"
#include "Misc/StringBuilder.h"
#include "HAL/IConsoleManager.h"
#include "Misc/AsciiSet.h"

DECLARE_CYCLE_STAT(TEXT("Broadcast"),STAT_StatsBroadcast,STATGROUP_StatSystem);
DECLARE_CYCLE_STAT(TEXT("Condense"),STAT_StatsCondense,STATGROUP_StatSystem);
DECLARE_DWORD_COUNTER_STAT(TEXT("Frame Messages"),STAT_StatFrameMessages,STATGROUP_StatSystem);
DECLARE_DWORD_COUNTER_STAT(TEXT("Total Frame Packets"),STAT_StatFramePackets,STATGROUP_StatSystem);
DECLARE_DWORD_COUNTER_STAT(TEXT("Frame Messages Condensed"),STAT_StatFramePacketsCondensed,STATGROUP_StatSystem);
DECLARE_MEMORY_STAT( TEXT("Stat Messages"), STAT_StatMessagesMemory, STATGROUP_StatSystem );

/*-----------------------------------------------------------------------------
	FStatConstants
-----------------------------------------------------------------------------*/

const FName FStatConstants::NAME_ThreadRoot = "ThreadRoot";
const char* FStatConstants::ThreadGroupName = STAT_GROUP_TO_FStatGroup( STATGROUP_Threads )::GetGroupName();
const FName FStatConstants::NAME_ThreadGroup = FStatConstants::ThreadGroupName;
const FName FStatConstants::RAW_SecondsPerCycle = FStatNameAndInfo( GET_STATFNAME( STAT_SecondsPerCycle ), true ).GetRawName();
const FName FStatConstants::NAME_NoCategory = FName("STATCAT_None");

const FString FStatConstants::StatsFileExtension = TEXT( ".uestats" );
const FString FStatConstants::StatsFileRawExtension = TEXT( ".uestatsraw" );

const FString FStatConstants::ThreadNameMarker = TEXT( "Thread_" );

const FName FStatConstants::RAW_EventWaitWithId = FStatNameAndInfo( GET_STATFNAME( STAT_EventWaitWithId ), true ).GetRawName();
const FName FStatConstants::RAW_EventTriggerWithId = FStatNameAndInfo( GET_STATFNAME( STAT_EventTriggerWithId ), true ).GetRawName();
const FName FStatConstants::RAW_NamedMarker = FStatNameAndInfo( GET_STATFNAME( STAT_NamedMarker ), true ).GetRawName(); 

const FStatNameAndInfo FStatConstants::AdvanceFrame = FStatNameAndInfo( NAME_AdvanceFrame, "", "", TEXT( "" ), EStatDataType::ST_int64, true, false, false );

extern bool GRenderStats;

/*-----------------------------------------------------------------------------
	FRawStatStackNode
-----------------------------------------------------------------------------*/

FRawStatStackNode::FRawStatStackNode(FRawStatStackNode const& Other)
	: Meta(Other.Meta)
{
	Children.Empty(Other.Children.Num());
	for (TMap<FName, FRawStatStackNode*>::TConstIterator It(Other.Children); It; ++It)
	{
		Children.Add(It.Key(), new FRawStatStackNode(*It.Value()));
	}
}

void FRawStatStackNode::MergeMax(FRawStatStackNode const& Other)
{
	check(Meta.NameAndInfo.GetRawName() == Other.Meta.NameAndInfo.GetRawName());
	if (Meta.NameAndInfo.GetField<EStatDataType>() != EStatDataType::ST_None && Meta.NameAndInfo.GetField<EStatDataType>() != EStatDataType::ST_FName)
	{
		FStatsUtils::AccumulateStat(Meta, Other.Meta, EStatOperation::MaxVal);
	}
	for (TMap<FName, FRawStatStackNode*>::TConstIterator It(Other.Children); It; ++It)
	{
		FRawStatStackNode* Child = Children.FindRef(It.Key());
		if (Child)
		{
			Child->MergeMax(*It.Value());
		}
		else
		{
			Children.Add(It.Key(), new FRawStatStackNode(*It.Value()));
		}
	}
}

void FRawStatStackNode::MergeAdd(FRawStatStackNode const& Other)
{
	check(Meta.NameAndInfo.GetRawName() == Other.Meta.NameAndInfo.GetRawName());
	if (Meta.NameAndInfo.GetField<EStatDataType>() != EStatDataType::ST_None && Meta.NameAndInfo.GetField<EStatDataType>() != EStatDataType::ST_FName)
	{
		FStatsUtils::AccumulateStat(Meta, Other.Meta, EStatOperation::Add);
	}
	for (TMap<FName, FRawStatStackNode*>::TConstIterator It(Other.Children); It; ++It)
	{
		FRawStatStackNode* Child = Children.FindRef(It.Key());
		if (Child)
		{
			Child->MergeAdd(*It.Value());
		}
		else
		{
			Children.Add(It.Key(), new FRawStatStackNode(*It.Value()));
		}
	}
}

void FRawStatStackNode::Divide(uint32 Div)
{
	if (Meta.NameAndInfo.GetField<EStatDataType>() != EStatDataType::ST_None && Meta.NameAndInfo.GetField<EStatDataType>() != EStatDataType::ST_FName)
	{
		FStatsUtils::DivideStat(Meta, Div);
	}
	for (TMap<FName, FRawStatStackNode*>::TIterator It(Children); It; ++It)
	{
		It.Value()->Divide(Div);
	}
}

void FRawStatStackNode::CullByCycles( int64 MinCycles )
{
	FRawStatStackNode* Culled = nullptr;
	const int32 NumChildren = Children.Num();
	for (TMap<FName, FRawStatStackNode*>::TIterator It( Children ); It; ++It)
	{
		FRawStatStackNode* Child = It.Value();
		const int64 ChildCycles = Child->Meta.GetValue_Duration();
		if (FromPackedCallCountDuration_Duration( Child->Meta.GetValue_int64() ) < MinCycles)
		{
			// Don't accumulate if we have just one child.
			if (NumChildren > 1)
			{
				if (!Culled)
				{
					Culled = new FRawStatStackNode( FStatMessage( NAME_OtherChildren, EStatDataType::ST_int64, nullptr, nullptr, nullptr, true, true, false ) );
					Culled->Meta.NameAndInfo.SetFlag( EStatMetaFlags::IsPackedCCAndDuration, true );
					Culled->Meta.Clear();
				}
				FStatsUtils::AccumulateStat( Culled->Meta, Child->Meta, EStatOperation::Add, true );
				delete Child;
				It.RemoveCurrent();
			}
			else
			{
				// Remove children.
				for (TMap<FName, FRawStatStackNode*>::TIterator InnerIt( Child->Children ); InnerIt; ++InnerIt)
				{
					delete InnerIt.Value();
					InnerIt.RemoveCurrent();
				}
			}
		}
		else if (NumChildren > 0)
		{
			Child->CullByCycles( MinCycles );
		}
	}
	if (Culled)
	{
		Children.Add( NAME_OtherChildren, Culled );
	}
}

void FRawStatStackNode::CullByDepth( int32 NoCullLevels )
{
	FRawStatStackNode* Culled = nullptr;
	const int32 NumChildren = Children.Num();
	for (TMap<FName, FRawStatStackNode*>::TIterator It( Children ); It; ++It)
	{
		FRawStatStackNode* Child = It.Value();
		if (NoCullLevels < 1)
		{
			delete Child;
			It.RemoveCurrent();
		}
		else
		{
			Child->CullByDepth( NoCullLevels - 1 );
		}
	}
}

int64 FRawStatStackNode::ChildCycles() const
{
	int64 Total = 0;
	for (TMap<FName, FRawStatStackNode*>::TConstIterator It(Children); It; ++It)
	{
		FRawStatStackNode const* Child = It.Value();
		Total += FromPackedCallCountDuration_Duration(Child->Meta.GetValue_int64());
	}
	return Total;
}

void FRawStatStackNode::AddNameHierarchy(int32 CurrentPrefixDepth)
{
	if (Children.Num())
	{
		if (Children.Num() > 1 && Meta.NameAndInfo.GetField<EStatDataType>() == EStatDataType::ST_int64 && Meta.NameAndInfo.GetFlag(EStatMetaFlags::IsPackedCCAndDuration))
		{
			TArray<FRawStatStackNode*> NewChildren;
			TArray<FRawStatStackNode*> ChildArray;
			Children.GenerateValueArray(ChildArray);
			ChildArray.Sort(FStatNameComparer<FRawStatStackNode>());
			TArray<TArray<FName> > ChildNames;
			ChildNames.Empty(ChildArray.Num());
			NewChildren.Empty(ChildArray.Num());

			FString Name;
			for (int32 Index = 0; Index < ChildArray.Num(); Index++)
			{
				FRawStatStackNode& Child = *ChildArray[Index];
				ChildNames.AddDefaulted();
				TArray<FName>& ParsedNames = ChildNames[Index];

				TArray<FString> Parts;
				Name = Child.Meta.NameAndInfo.GetRawName().ToString();
				if (Name.StartsWith(TEXT("//")))
				{
					// we won't add hierarchy for grouped stats
					ParsedNames.Add(Child.Meta.NameAndInfo.GetRawName());
				}
				else
				{
					Name.ReplaceInline(TEXT("/"), TEXT("."));
					Name.ParseIntoArray(Parts, TEXT("."), true);
					check(Parts.Num());
					ParsedNames.Empty(Parts.Num());
					for (int32 PartIndex = 0; PartIndex < Parts.Num(); PartIndex++)
					{
						ParsedNames.Add(*Parts[PartIndex]);
					}
				}
			}

			int32 StartIndex = 0;

			while (StartIndex < ChildArray.Num())
			{
				int32 MaxParts = ChildNames[StartIndex].Num() - CurrentPrefixDepth;
				int32 NumWithCommonRoot = 1;
				if (MaxParts > 0)
				{
					for (int32 TestIndex = StartIndex + 1; TestIndex < ChildArray.Num(); TestIndex++)
					{
						if (CurrentPrefixDepth >= ChildNames[TestIndex].Num() || ChildNames[TestIndex][CurrentPrefixDepth] != ChildNames[StartIndex][CurrentPrefixDepth])
						{
							break;
						}
						NumWithCommonRoot++;
					}
				}
				if (NumWithCommonRoot < 2 || MaxParts < 1)
				{
					ChildArray[StartIndex]->AddNameHierarchy();
					NewChildren.Add(ChildArray[StartIndex]);
					StartIndex++;
					continue;
				}
				int32 MaxCommonality = CurrentPrefixDepth + 1;
				bool bOk = true;
				for (int32 TestDepth = CurrentPrefixDepth + 1; bOk && TestDepth < ChildNames[StartIndex].Num(); TestDepth++)
				{
					for (int32 TestIndex = StartIndex + 1; bOk && TestIndex < StartIndex + NumWithCommonRoot; TestIndex++)
					{
						if (TestDepth >= ChildNames[TestIndex].Num() || ChildNames[TestIndex][TestDepth] != ChildNames[StartIndex][TestDepth])
						{
							bOk = false;
						}
					}
					if (bOk)
					{
						MaxCommonality = TestDepth + 1;
					}
				}
				FString NewName("NameFolder//");
				for (int32 TestDepth = 0; TestDepth < MaxCommonality; TestDepth++)
				{
					NewName += ChildNames[StartIndex][TestDepth].ToString();
					NewName += TEXT(".");
				}
				NewName += TEXT("..");
				FStatMessage Group(ChildArray[StartIndex]->Meta);
				FName NewFName(*NewName);
				Group.NameAndInfo.SetRawName(NewFName);
				Group.Clear();
				FRawStatStackNode* NewNode = new FRawStatStackNode(Group);
				NewChildren.Add(NewNode);
				for (int32 TestIndex = StartIndex; TestIndex < StartIndex + NumWithCommonRoot; TestIndex++)
				{
					FStatsUtils::AccumulateStat(NewNode->Meta, ChildArray[TestIndex]->Meta, EStatOperation::Add, true);
					NewNode->Children.Add(ChildArray[TestIndex]->Meta.NameAndInfo.GetRawName(), ChildArray[TestIndex]);
				}
				NewNode->AddNameHierarchy(MaxCommonality);
				StartIndex += NumWithCommonRoot;
			}
			Children.Empty(NewChildren.Num());
			for (int32 Index = 0; Index < NewChildren.Num(); Index++)
			{
				Children.Add(NewChildren[Index]->Meta.NameAndInfo.GetRawName(), NewChildren[Index]);
			}
		}
		else
		{
			for (TMap<FName, FRawStatStackNode*>::TIterator It(Children); It; ++It)
			{
				FRawStatStackNode* Child = It.Value();
				Child->AddNameHierarchy();
			}
		}
	}
}

void FRawStatStackNode::AddSelf()
{
	if (Children.Num())
	{
		if (Meta.NameAndInfo.GetField<EStatDataType>() == EStatDataType::ST_int64 && Meta.NameAndInfo.GetFlag(EStatMetaFlags::IsPackedCCAndDuration))
		{
			FStatMessage Self(Meta);
			int64 MyTime = Self.GetValue_Duration();
			MyTime -= ChildCycles();
			if (MyTime > 0)
			{
				Self.GetValue_int64() = ToPackedCallCountDuration(
					FromPackedCallCountDuration_CallCount(Self.GetValue_int64()),
					(uint32)MyTime);
				Self.NameAndInfo.SetRawName(NAME_Self);
				Children.Add(NAME_Self, new FRawStatStackNode(Self));
			}
		}
		for (TMap<FName, FRawStatStackNode*>::TIterator It(Children); It; ++It)
		{
			FRawStatStackNode* Child = It.Value();
			Child->AddSelf();
		}
	}
}

void FRawStatStackNode::DebugPrint(TCHAR const* Filter, int32 InMaxDepth, int32 Depth) const
{
	if (Depth <= InMaxDepth)
	{
		if (!Filter || !*Filter)
		{
			FString TmpDebugStr = FStatsUtils::DebugPrint(Meta);
			UE_LOGF(LogStats, Log, "%ls%ls", FCString::Spc(FMath::Min(TCStringSpcHelper<TCHAR>::MAX_SPACES, Depth*2)), *TmpDebugStr);
		}

		static int64 MinPrint = -1;
		if (Children.Num())
		{
			TArray<FRawStatStackNode*> ChildArray;
			Children.GenerateValueArray(ChildArray);
			ChildArray.Sort( FStatDurationComparer<FRawStatStackNode>() );
			for (int32 Index = 0; Index < ChildArray.Num(); Index++)
			{
				if (ChildArray[Index]->Meta.GetValue_Duration() < MinPrint)
				{
					break;
				}
				if (Filter && *Filter)
				{
					if (ChildArray[Index]->Meta.NameAndInfo.GetRawName().ToString().Contains(Filter))
					{
						ChildArray[Index]->DebugPrint(nullptr, InMaxDepth, 0);
					}
					else
					{
						ChildArray[Index]->DebugPrint(Filter, InMaxDepth, 0);
					}
				}
				else
				{
					ChildArray[Index]->DebugPrint(Filter, InMaxDepth, Depth + 1);
				}
			}
		}
	}
}

void FRawStatStackNode::DebugPrintLeafFilter(TCHAR const* Filter) const
{
	TArray<FString> Stack;
	DebugPrintLeafFilterInner(Filter, 0, Stack);
}

void FRawStatStackNode::DebugPrintLeafFilterInner(TCHAR const* Filter, int32 Depth, TArray<FString>& Stack) const
{
	{
		FString TmpDebugStr = FStatsUtils::DebugPrint(Meta);
		Stack.Add(TmpDebugStr);
	}
	if (!Filter || !*Filter)
	{
		int32 Offset = 1 + Depth - Stack.Num();
		check(Offset >= 0);
		for (int32 Index = 0; Index < Stack.Num(); Index++)
		{
			UE_LOGF(LogStats, Log, "%ls%ls", FCString::Spc((Index + Offset) * 2), *Stack[Index]);
		}
		Stack.Reset();
	}
	else
	{
		static int64 MinPrint = -1;
		if (Children.Num())
		{
			TArray<FRawStatStackNode*> ChildArray;
			Children.GenerateValueArray(ChildArray);
			ChildArray.Sort(FStatDurationComparer<FRawStatStackNode>());
			for (int32 Index = 0; Index < ChildArray.Num(); Index++)
			{
				if (ChildArray[Index]->Meta.GetValue_Duration() < MinPrint)
				{
					break;
				}
				if (ChildArray[Index]->Meta.NameAndInfo.GetRawName().ToString().Contains(Filter))
				{
					ChildArray[Index]->DebugPrintLeafFilterInner(nullptr, Depth + 1, Stack);
				}
				else
				{
					ChildArray[Index]->DebugPrintLeafFilterInner(Filter, Depth + 1, Stack);
				}
			}
		}
		if (Stack.Num())
		{
			Stack.Pop();
		}
	}
}


void FRawStatStackNode::Encode(TArray<FStatMessage>& OutStats) const
{
	FStatMessage& NewStat = OutStats.Add_GetRef(Meta);
	if (Children.Num())
	{
		NewStat.NameAndInfo.SetField<EStatOperation>(EStatOperation::ChildrenStart);
		for (TMap<FName, FRawStatStackNode*>::TConstIterator It(Children); It; ++It)
		{
			FRawStatStackNode const* Child = It.Value();
			Child->Encode(OutStats);
		}
		FStatMessage& EndStat = OutStats.Emplace_GetRef(Meta);
		EndStat.NameAndInfo.SetField<EStatOperation>(EStatOperation::ChildrenEnd);
	}
	else
	{
		NewStat.NameAndInfo.SetField<EStatOperation>(EStatOperation::Leaf);
	}
}

TLockFreeFixedSizeAllocator<sizeof(FRawStatStackNode), PLATFORM_CACHE_LINE_SIZE>& GetRawStatStackNodeAllocator()
{
	static TLockFreeFixedSizeAllocator<sizeof(FRawStatStackNode), PLATFORM_CACHE_LINE_SIZE> TheAllocator;
	return TheAllocator;
}

void* FRawStatStackNode::operator new(size_t Size)
{
	checkSlow(Size == sizeof(FRawStatStackNode));
	return GetRawStatStackNodeAllocator().Allocate();
}

void FRawStatStackNode::operator delete(void *RawMemory)
{
	GetRawStatStackNodeAllocator().Free(RawMemory);
}	

/*-----------------------------------------------------------------------------
	FComplexRawStatStackNode
-----------------------------------------------------------------------------*/

TLockFreeFixedSizeAllocator<sizeof(FComplexRawStatStackNode), PLATFORM_CACHE_LINE_SIZE>& GetRawStatStackNodeAllocatorEx()
{	
	static TLockFreeFixedSizeAllocator<sizeof(FComplexRawStatStackNode), PLATFORM_CACHE_LINE_SIZE> TheAllocatorComplex;
	return TheAllocatorComplex;
}

void* FComplexRawStatStackNode::operator new(size_t Size)
{
	checkSlow(Size == sizeof(FComplexRawStatStackNode));
	return GetRawStatStackNodeAllocatorEx().Allocate();
}

void FComplexRawStatStackNode::operator delete(void *RawMemory)
{
	GetRawStatStackNodeAllocatorEx().Free(RawMemory);
}

FComplexRawStatStackNode::FComplexRawStatStackNode( const FComplexRawStatStackNode& Other )
	: ComplexStat(Other.ComplexStat)
{
	Children.Empty(Other.Children.Num());
	for (auto It = Other.Children.CreateConstIterator(); It; ++It)
	{
		Children.Add(It.Key(), new FComplexRawStatStackNode(*It.Value()));
	}
}

FComplexRawStatStackNode::FComplexRawStatStackNode( const FRawStatStackNode& Other )
	: ComplexStat(Other.Meta)
{
	Children.Empty(Other.Children.Num());
	for (auto It = Other.Children.CreateConstIterator(); It; ++It)
	{
		Children.Add(It.Key(), new FComplexRawStatStackNode(*It.Value()));
	}
}

void FComplexRawStatStackNode::MergeAddAndMinMax( const FRawStatStackNode& Other )
{
	FComplexStatUtils::AddAndMinMax( ComplexStat, Other.Meta, EComplexStatField::IncSum, EComplexStatField::IncMax, EComplexStatField::IncMin );

	for (auto It = Other.Children.CreateConstIterator(); It; ++It)
	{
		FComplexRawStatStackNode* Child = Children.FindRef(It.Key());
		if (Child)
		{
			Child->MergeAddAndMinMax(*It.Value());
		}
		else
		{
			Children.Add(It.Key(), new FComplexRawStatStackNode(*It.Value()));
		}
	}
}

void FComplexRawStatStackNode::Divide(uint32 Div)
{
	if (ComplexStat.NameAndInfo.GetField<EStatDataType>() != EStatDataType::ST_None && ComplexStat.NameAndInfo.GetField<EStatDataType>() != EStatDataType::ST_FName)
	{
		FComplexStatUtils::DivideStat( ComplexStat, Div, EComplexStatField::IncSum, EComplexStatField::IncAve );
	}
	for (auto It = Children.CreateIterator(); It; ++It)
	{
		It.Value()->Divide(Div);
	}
}

void FComplexRawStatStackNode::CullByCycles( int64 MinCycles )
{
	FComplexRawStatStackNode* Culled = nullptr;
	const int32 NumChildren = Children.Num();
	for (auto It = Children.CreateIterator(); It; ++It)
	{
		FComplexRawStatStackNode* Child = It.Value();
		const int64 ChildCycles = Child->ComplexStat.GetValue_Duration( EComplexStatField::IncAve );
		if (ChildCycles < MinCycles)
		{
			// Don't accumulate if we have just one child.
			if (NumChildren > 1)
			{	
				delete Child;
				It.RemoveCurrent();
			}
			else
			{
				// Remove children.
				for (auto ChildIt = Child->Children.CreateIterator(); ChildIt; ++ChildIt)
				{
					delete ChildIt.Value();
					ChildIt.RemoveCurrent();
				}
			}
		}
		else if (NumChildren > 0)
		{
			Child->CullByCycles( MinCycles );
		}
	}
	if (Culled)
	{
		Children.Add( NAME_OtherChildren, Culled );
	}
}

void FComplexRawStatStackNode::CullByDepth( int32 NoCullLevels )
{
	FComplexRawStatStackNode* Culled = nullptr;
	const int32 NumChildren = Children.Num();
	for (auto It = Children.CreateIterator(); It; ++It)
	{
		FComplexRawStatStackNode* Child = It.Value();
		if (NoCullLevels < 1)
		{
			delete Child;
			It.RemoveCurrent();
		}
		else
		{
			Child->CullByDepth( NoCullLevels - 1 );
		}
	}
}

void FComplexRawStatStackNode::CopyExclusivesFromSelf()
{
	if (Children.Num())
	{
		const FComplexRawStatStackNode* SelfStat = Children.FindRef( NAME_Self );
		if (SelfStat)
		{
			ComplexStat.GetValue_int64( EComplexStatField::ExcAve ) = SelfStat->ComplexStat.GetValue_int64( EComplexStatField::IncAve );
			ComplexStat.GetValue_int64( EComplexStatField::ExcMax ) = SelfStat->ComplexStat.GetValue_int64( EComplexStatField::IncMax );
		}

		for (auto It = Children.CreateIterator(); It; ++It)
		{
			FComplexRawStatStackNode* Child = It.Value();
			Child->CopyExclusivesFromSelf();
		}
	}
}

/*-----------------------------------------------------------------------------
	FStatsThreadState
-----------------------------------------------------------------------------*/

void FStatPacketArray::Empty()
{
	FStatsThreadState& State = FStatsThreadState::GetLocalState();

	for (int32 Index = 0; Index < Packets.Num(); Index++)
	{
		State.NumStatMessages.Subtract(Packets[Index]->StatMessages.Num());
		delete Packets[Index];
	}
	Packets.Empty();
}

FStatsThreadState::FStatsThreadState(int32 InHistoryFrames)
	: HistoryFrames(InHistoryFrames)
	, LastFullFrameMetaAndNonFrame(-1)
	, LastFullFrameProcessed(-1)
	, TotalNumStatMessages(0)
	, MaxNumStatMessages(0)
	, bFindMemoryExtensiveStats(false)
	, CurrentGameFrame(1)
	, CurrentRenderFrame(1)
	, CurrentEndOfPipeFrame(1)
{
}

FStatsThreadState& FStatsThreadState::GetLocalState()
{
	static FStatsThreadState Singleton;
	return Singleton;
}

int64 FStatsThreadState::GetOldestValidFrame() const
{
	int64 Result = -1;
	for (auto It = GoodFrames.CreateConstIterator(); It; ++It)
	{
		if ((Result == -1 || *It < Result) && *It <= LastFullFrameMetaAndNonFrame)
		{
			Result = *It;
		}
	}
	return Result;
}

int64 FStatsThreadState::GetLatestValidFrame() const
{
	int64 Result = -1;
	for (auto It = GoodFrames.CreateConstIterator(); It; ++It)
	{
		if (*It > Result && *It <= LastFullFrameMetaAndNonFrame)
		{
			Result = *It;
		}
	}
	return Result;
}

static TAutoConsoleVariable<int32> CVarSpewStatsSpam(
	TEXT("stats.SpewSpam"),
	0,
	TEXT("If set to 1, periodically prints a profile of messages coming into the stats system. Messages should be minimized to cut down on overhead.")
	);


void FStatsThreadState::ScanForAdvance(const FStatMessagesArray& Data)
{
	if (CVarSpewStatsSpam.GetValueOnAnyThread())
	{
		static const int32 FramesPerSpew = 300;
		static TMap<FName, int32> Profile;
		static uint64 LastFrame = GFrameCounter;
		for (const FStatMessage& Item : Data)
		{
			FName ItemName = Item.NameAndInfo.GetRawName();
			Profile.FindOrAdd(ItemName)++;
		}
		if (GFrameCounter > LastFrame + FramesPerSpew)
		{
			LastFrame = GFrameCounter;
			Profile.ValueSort(TGreater<>());
			UE_LOGF(LogStats, Log, "---- stats spam profile -------------");
			for (TPair<FName, int32> Pair : Profile)
			{
				float PerFrame = float(Pair.Value) / float(FramesPerSpew);

				if (PerFrame < 50.0f)
				{
					break;
				}
				UE_LOGF(LogStats, Log, "       %6.0f    %ls", PerFrame, *Pair.Key.ToString());
			}
			Profile.Reset();
		}
	}
	for (const FStatMessage& Item : Data)
	{
		EStatOperation::Type Op = Item.NameAndInfo.GetField<EStatOperation>();
		if (Op == EStatOperation::AdvanceFrameEventGameThread)
		{
			check(Item.NameAndInfo.GetField<EStatDataType>() == EStatDataType::ST_int64);
			int64 NewGameFrame = Item.GetValue_int64();

			if (NewGameFrame < 0)
			{
				NewGameFrame = -NewGameFrame;
				BadFrames.Add(NewGameFrame - 1);
			}
			if (CurrentGameFrame > STAT_FRAME_SLOP && CurrentGameFrame + 1 != NewGameFrame)
			{
				// this packet has multiple advances in it. They are all bad.
				check(CurrentGameFrame + 1 < NewGameFrame);
				for (int64 Frame = CurrentGameFrame + 1; Frame <= NewGameFrame; Frame ++)
				{
					BadFrames.Add(Frame - 1);
				}
			}
			CurrentGameFrame = NewGameFrame;
		}
		else if (Op == EStatOperation::AdvanceFrameEventRenderThread)
		{
			check(Item.NameAndInfo.GetField<EStatDataType>() == EStatDataType::ST_int64);
			int64 NewRenderFrame = Item.GetValue_int64();

			if (NewRenderFrame < 0)
			{
				NewRenderFrame = -NewRenderFrame;
				BadFrames.Add(NewRenderFrame - 1);
			}

			if (CurrentRenderFrame > STAT_FRAME_SLOP && CurrentRenderFrame + 1 != NewRenderFrame)
			{
				// this packet has multiple advances in it. They are all bad.
				check(CurrentRenderFrame + 1 < NewRenderFrame);
				for (int64 Frame = CurrentRenderFrame + 1; Frame <= NewRenderFrame; Frame ++)
				{
					BadFrames.Add(Frame - 1);
				}
			}
			CurrentRenderFrame = NewRenderFrame;
		}
		else if (Op == EStatOperation::AdvanceFrameEventEndOfPipe)
		{
			check(Item.NameAndInfo.GetField<EStatDataType>() == EStatDataType::ST_int64);
			int64 NewEndOfPipeFrame = Item.GetValue_int64();

			if (NewEndOfPipeFrame < 0)
			{
				NewEndOfPipeFrame = -NewEndOfPipeFrame;
				BadFrames.Add(NewEndOfPipeFrame - 1);
			}

			if (CurrentEndOfPipeFrame > STAT_FRAME_SLOP && CurrentEndOfPipeFrame + 1 != NewEndOfPipeFrame)
			{
				// this packet has multiple advances in it. They are all bad.
				check(CurrentEndOfPipeFrame + 1 < NewEndOfPipeFrame);
				for (int64 Frame = CurrentEndOfPipeFrame + 1; Frame <= NewEndOfPipeFrame; Frame++)
				{
					BadFrames.Add(Frame - 1);
				}
			}
			CurrentEndOfPipeFrame = NewEndOfPipeFrame;
		}
	}

	// We don't care about bad frame when the raw stats are active.
	if( FThreadStats::bIsRawStatsActive )
	{
		BadFrames.Empty();
	}
}

void FStatsThreadState::ScanForAdvance(FStatPacketArray& NewData)
{
	if (!FThreadStats::WillEverCollectData())
	{
		return;
	}

	uint32 Count = 0;
	for (FStatPacket* Packet : NewData.Packets)
	{
		switch (Packet->ThreadType)
		{
			case EThreadType::Renderer:
				Packet->AssignFrame( CurrentRenderFrame );
				break;

			case EThreadType::Game:
				Packet->AssignFrame( CurrentGameFrame );
				break;

			case EThreadType::EndOfPipe:
				Packet->AssignFrame( CurrentEndOfPipeFrame );
				break;

			case EThreadType::Other:
				// @see FThreadStats::DetectAndUpdateCurrentGameFrame 
				break;

			default:
				checkf( 0, TEXT( "Unknown thread type" ) );
		}

		const FStatMessagesArray& Data = Packet->StatMessages;
		ScanForAdvance(Data);
		Count += Data.Num();
	}
	INC_DWORD_STAT_BY(STAT_StatFramePackets, NewData.Packets.Num());
	INC_DWORD_STAT_BY(STAT_StatFrameMessages, Count);
}

void FStatsThreadState::ProcessMetaDataOnly(TArray64<FStatMessage>& Data)
{
	for (int64 Index = 0; Index < Data.Num(); Index++)
	{
		FStatMessage& Item = Data[Index];
		EStatOperation::Type Op = Item.NameAndInfo.GetField<EStatOperation>();
		check(Op == EStatOperation::SetLongName);
		FindOrAddMetaData(Item);
	}
}

void FStatsThreadState::ToggleFindMemoryExtensiveStats()
{
	bFindMemoryExtensiveStats = !bFindMemoryExtensiveStats;
	UE_LOGF(LogStats, Log, "bFindMemoryExtensiveStats is %ls now", bFindMemoryExtensiveStats?TEXT("enabled"):TEXT("disabled"));
}

void FStatsThreadState::ProcessNonFrameStats(FStatMessagesArray& Data, TSet<FName>* NonFrameStatsFound)
{
	for (FStatMessage& Item : Data)
	{
		check(Item.NameAndInfo.GetFlag(EStatMetaFlags::DummyAlwaysOne));  // we should never be sending short names to the stats any more
		EStatOperation::Type Op = Item.NameAndInfo.GetField<EStatOperation>();
		check(Op != EStatOperation::SetLongName);
		if (!Item.NameAndInfo.GetFlag(EStatMetaFlags::ShouldClearEveryFrame))
		{
			if (!(
				Op != EStatOperation::CycleScopeStart && 
				Op != EStatOperation::CycleScopeEnd &&
				Op != EStatOperation::ChildrenStart &&
				Op != EStatOperation::ChildrenEnd &&
				Op != EStatOperation::Leaf &&
				Op != EStatOperation::AdvanceFrameEventGameThread &&
				Op != EStatOperation::AdvanceFrameEventRenderThread &&
				Op != EStatOperation::AdvanceFrameEventEndOfPipe
				))
			{
				UE_LOGF(LogStats, Fatal, "Stat %ls was not cleared every frame, but was used with a scope cycle counter.", *Item.NameAndInfo.GetRawName().ToString() );
			}
			else if (Op == EStatOperation::SpecialMessageMarker)
			{
				// Ignore special messages, they shouldn't be treated as regular stats messages.
			}
			else
			{
				FStatMessage* Result = NotClearedEveryFrame.Find(Item.NameAndInfo.GetRawName());
				if (!Result)
				{
					UE_LOGF(
						LogStats, Error,
						"Stat %ls was cleared every frame, but we don't have metadata for it. Data loss." " (N.B. One cause for this can be duplicately named stats)",
						*Item.NameAndInfo.GetRawName().ToString()
					);
				}
				else
				{
					if (NonFrameStatsFound)
					{
						NonFrameStatsFound->Add(Item.NameAndInfo.GetRawName());
					}
					FStatsUtils::AccumulateStat(*Result, Item);
					Item = *Result; // now just write the accumulated value back into the stream
					check(Item.NameAndInfo.GetField<EStatOperation>() == EStatOperation::Set);
				}
			}
		}
	}
}

void FStatsThreadState::AddToHistoryAndEmpty(FStatPacketArray& NewData)
{
	if (!FThreadStats::WillEverCollectData())
	{
		NewData.Empty(); // delete the elements
		CondensedStackHistory.Empty();
		GoodFrames.Empty();
		BadFrames.Empty();
		NotClearedEveryFrame.Empty();
		ShortNameToLongName.Empty();
		Groups.Empty();
		History.Empty();
		EventsHistory.Empty();
		return;
	}

	for (FStatPacket* Packet : NewData.Packets)
	{
		int64 FrameNum = Packet->Frame;
		FStatPacketArray& Frame = History.FindOrAdd(FrameNum);
		Frame.Packets.Add(Packet);
		if (FrameNum <= LastFullFrameMetaAndNonFrame && LastFullFrameMetaAndNonFrame != -1)
		{
			// This packet was from an older frame. We process the non-frame stats immediately here
			// since the algorithm below assumes only new frames should be processed.
			ProcessNonFrameStats(Packet->StatMessages, nullptr);
		}
	}

	NewData.RemovePtrsButNoData(); // don't delete the elements

	// now deal with metadata and non-frame stats

	TArray<int64> Frames;
	History.GenerateKeyArray(Frames);
	Frames.Sort();

	int64 LatestFinishedFrame = FMath::Min<int64>(CurrentGameFrame, CurrentRenderFrame, CurrentEndOfPipeFrame) - 1;

	for (int64 FrameNum : Frames)
	{
		if (LastFullFrameMetaAndNonFrame < 0)
		{
			LastFullFrameMetaAndNonFrame = FrameNum - 1;
		}
		if (FrameNum <= LatestFinishedFrame && FrameNum == LastFullFrameMetaAndNonFrame + 1)
		{
			FStatPacketArray& Frame = History.FindChecked(FrameNum);

			FStatPacket const* PacketToCopyForNonFrame = nullptr;

			if( bFindMemoryExtensiveStats )
			{
				FindAndDumpMemoryExtensiveStats(Frame);
			}

			TSet<FName> NonFrameStatsFound;
			for (FStatPacket* Packet : Frame.Packets)
			{
				ProcessNonFrameStats(Packet->StatMessages, &NonFrameStatsFound);
				if (!PacketToCopyForNonFrame && Packet->ThreadType == EThreadType::Game)
				{
					PacketToCopyForNonFrame = Packet;
				}
			}
			// was this a good frame
			if (PacketToCopyForNonFrame && !BadFrames.Contains(FrameNum))
			{
				// add the non frame stats as a new last packet
				FStatPacket* NonFramePacket = nullptr;

				for (const TPair<FName, FStatMessage>& It : NotClearedEveryFrame)
				{
					if (!NonFrameStatsFound.Contains(It.Key)) // don't add stats that are updated during this frame, they would be redundant
					{
						if (!NonFramePacket)
						{
							NonFramePacket = new FStatPacket(*PacketToCopyForNonFrame);
							Frame.Packets.Add(NonFramePacket);
						}
						
						NonFramePacket->StatMessages.AddElement(It.Value);
					}
				}

				if(NonFramePacket)
				{
					NumStatMessages.Add(NonFramePacket->StatMessages.Num());
				}

				GoodFrames.Add(FrameNum);
			}
			LastFullFrameMetaAndNonFrame = FrameNum;
		}
	}

	const int64 NewLatestFrame = GetLatestValidFrame();

	if (NewLatestFrame > 0)
	{
		check(GoodFrames.Contains(NewLatestFrame));
		if (NewLatestFrame > LastFullFrameProcessed)
		{
			int64 FirstNewFrame = FMath::Max<int64>(GetOldestValidFrame(), LastFullFrameProcessed + 1);

			// let people know
			{
				SCOPE_CYCLE_COUNTER(STAT_StatsBroadcast);
				for (int64 Frame = FirstNewFrame; Frame <= NewLatestFrame; Frame++ )
				{
					if (IsFrameValid(Frame))
					{
						NewFrameDelegate.Broadcast(Frame);
						LastFullFrameProcessed = Frame;
					}
				}
			}
		}
	}

	const int64 MinFrameToKeep = LatestFinishedFrame - HistoryFrames;

	for (auto It = BadFrames.CreateIterator(); It; ++It)
	{
		const int64 ThisFrame = *It;
		if (ThisFrame <= LastFullFrameMetaAndNonFrame && ThisFrame < MinFrameToKeep)
		{
			It.RemoveCurrent();
		}
	}
	for (auto It = History.CreateIterator(); It; ++It)
	{
		const int64 ThisFrame = It.Key();
		if (ThisFrame <= LastFullFrameMetaAndNonFrame && ThisFrame < MinFrameToKeep)
		{
			It.RemoveCurrent();
		}
	}
	for (auto It = EventsHistory.CreateIterator(); It; ++It)
	{
		const int64 ThisFrame = It.Value().Frame;
		if (ThisFrame <= LastFullFrameProcessed && ThisFrame < MinFrameToKeep)
		{
			It.RemoveCurrent();
		}
	}
	for (auto It = CondensedStackHistory.CreateIterator(); It; ++It)
	{
		const int64 ThisFrame = It.Key();
		if (ThisFrame <= LastFullFrameProcessed && ThisFrame < MinFrameToKeep)
		{
			delete It.Value();
			It.RemoveCurrent();
		}
	}
	for (auto It = GoodFrames.CreateIterator(); It; ++It)
	{
		const int64 ThisFrame = *It;
		if (!History.Contains(ThisFrame) && !CondensedStackHistory.Contains(ThisFrame)) // if it isn't in the history anymore, it isn't good anymore
		{
			It.RemoveCurrent();
		}
	}
}


void FStatsThreadState::ProcessRawStats( FStatPacketArray& NewData )
{
	if( NewRawStatPacket.IsBound() )
	{
		// First process the enqueued raw stats.
		for (int32 Index = 0; Index < StartupRawStats.Packets.Num(); Index++)
		{
			const FStatPacket* StatPacket = StartupRawStats.Packets[Index];
			NewRawStatPacket.Broadcast(StatPacket);
		}
		StartupRawStats.Empty();

		// Now, process the raw stats.
		for (int32 Index = 0; Index < NewData.Packets.Num(); Index++)
		{
			const FStatPacket* StatPacket = NewData.Packets[Index];
			NewRawStatPacket.Broadcast(StatPacket);
		}

		// Now delete all the data.
		NewData.Empty();
	}
	else
	{
		// The delegate is not bound yet, so store the data, because we don't want to lose any data.
		for (int32 Index = 0; Index < NewData.Packets.Num(); Index++)
		{
			FStatPacket* StatPacket = NewData.Packets[Index];
			StartupRawStats.Packets.Add(StatPacket);
		}

		NewData.RemovePtrsButNoData(); // Don't delete the elements.
	}
}

void FStatsThreadState::ResetRawStats()
{
	// We no longer need any startup raw data.
	StartupRawStats.Empty();
}

void FStatsThreadState::ResetRegularStats()
{
	// We need to reset these values after switching from the raw stats to the regular.
	// !!CAUTION!!
	// This is a bit unsafe as we lose accumulator history.
	// Cycle counters and general counters should be just fine.
	LastFullFrameMetaAndNonFrame = -1;
	LastFullFrameProcessed = -1;
	History.Empty();
	CondensedStackHistory.Empty();
	GoodFrames.Empty();
	BadFrames.Empty();
}

void FStatsThreadState::UpdateStatMessagesMemoryUsage()
{
	const int32 CurrentNumStatMessages = NumStatMessages.GetValue();
	MaxNumStatMessages = FMath::Max(MaxNumStatMessages,CurrentNumStatMessages);

	TotalNumStatMessages += CurrentNumStatMessages;
	SET_MEMORY_STAT( STAT_StatMessagesMemory, CurrentNumStatMessages*sizeof(FStatMessage) );

	if( FThreadStats::bIsRawStatsActive )
	{
		FGameThreadStatsData* ToGame = new FGameThreadStatsData(true, GRenderStats);

		const double InvMB = 1.0f / 1024.0f / 1024.0f;

		// Format lines to be displayed on the hud.
		const FString Current = FString::Printf( TEXT("Current: %.1f"), float(InvMB*double(CurrentNumStatMessages*sizeof(FStatMessage))) );
		const FString Max = FString::Printf( TEXT("Max: %.1f"), float(InvMB*double((int64)MaxNumStatMessages*sizeof(FStatMessage))) );
		const FString Total = FString::Printf( TEXT("Total: %.1f") , float(InvMB*double(TotalNumStatMessages*sizeof(FStatMessage))) );

		UE_LOGF(LogStats, Verbose, "%ls, %ls, %ls", *Current, *Max, *Total );

		ToGame->GroupDescriptions.Add( TEXT("RawStats memory usage (MB)") );
		ToGame->GroupDescriptions.Add( Current );
		ToGame->GroupDescriptions.Add( Max );
		ToGame->GroupDescriptions.Add( Total );

		FSimpleDelegateGraphTask::CreateAndDispatchWhenReady
		(
			FSimpleDelegateGraphTask::FDelegate::CreateRaw(&FLatestGameThreadStatsData::Get(), &FLatestGameThreadStatsData::NewData, ToGame),
			TStatId(), nullptr, ENamedThreads::GameThread
		);
	}
}

void FStatsThreadState::GetInclusiveAggregateStackStats(
	int64 TargetFrame, 
	TArray<FStatMessage>& OutStats, 
	IItemFilter* Filter /*= nullptr*/, 
	bool bAddNonStackStats /*= true*/, 
	TMap<FName, TArray<FStatMessage>>* OptionalOutThreadBreakdownMap /*= nullptr */) const
{
	const TArray<FStatMessage>& CondensedMessages = GetCondensedHistory( TargetFrame );
	GetInclusiveAggregateStackStats( CondensedMessages, OutStats, Filter, bAddNonStackStats, OptionalOutThreadBreakdownMap );
}

void FStatsThreadState::GetInclusiveAggregateStackStats( 
	const TArray<FStatMessage>& CondensedMessages, 
	TArray<FStatMessage>& OutStats, 
	IItemFilter* Filter /*= nullptr*/, 
	bool bAddNonStackStats /*= true*/, 
	TMap<FName, TArray<FStatMessage>>* OptionalOutThreadBreakdownMap /*= nullptr */ ) const
{
	struct FTimeInfo
	{
		int32 StartCalls;
		int32 StopCalls;
		int32 Recursion;
		FTimeInfo()
			: StartCalls(0)
			, StopCalls(0)
			, Recursion(0)
		{

		}
	};
	TMap<FName, FTimeInfo> Timing;
	TMap<FName, FStatMessage> ThisFrameMetaData;
	TMap<FName, TMap<FName, FStatMessage>> ThisFrameMetaDataPerThread;
	TMap<FName, FStatMessage> ThreadStarts;
	TMap<FName, FStatMessage> ThreadEnds;
	TMap<FName, FStatMessage>* ThisFrameMetaDataPerThreadPtr = nullptr;
	int32 Depth = 0;
	for (int32 Index = 0; Index < CondensedMessages.Num(); Index++)
	{
		FStatMessage const& Item = CondensedMessages[Index];

		//Need to get thread root first regardless of filter
		if(OptionalOutThreadBreakdownMap)
		{
			EStatOperation::Type Op = Item.NameAndInfo.GetField<EStatOperation>();
			if (Op == EStatOperation::ChildrenStart)
			{
				if(Depth++ == 1)
				{
					FName LongName = Item.NameAndInfo.GetRawName();
					check(ThisFrameMetaDataPerThreadPtr == nullptr);
					ThreadStarts.Add(LongName, Item);
					ThisFrameMetaDataPerThreadPtr = &ThisFrameMetaDataPerThread.FindOrAdd(LongName);
				}
			}
			else if (Op == EStatOperation::ChildrenEnd)
			{
				if(--Depth == 1)
				{
					FName LongName = Item.NameAndInfo.GetRawName();
					ThreadEnds.Add(LongName, Item);
					check(ThisFrameMetaDataPerThreadPtr);
					ThisFrameMetaDataPerThreadPtr = nullptr;
				}
			}
		}

		if (!Filter || Filter->Keep(Item))
		{
			FName LongName = Item.NameAndInfo.GetRawName();

			EStatOperation::Type Op = Item.NameAndInfo.GetField<EStatOperation>();
			if ((Op == EStatOperation::ChildrenStart || Op == EStatOperation::ChildrenEnd ||  Op == EStatOperation::Leaf) && Item.NameAndInfo.GetFlag(EStatMetaFlags::IsCycle))
			{

				FStatMessage* Result = ThisFrameMetaData.Find(LongName);
				if (!Result)
				{
					Result = &ThisFrameMetaData.Add(LongName, Item);
					Result->NameAndInfo.SetField<EStatOperation>(EStatOperation::Set);
					Result->NameAndInfo.SetFlag(EStatMetaFlags::IsPackedCCAndDuration, true);
					Result->Clear();
				}

				if(Depth && ThisFrameMetaDataPerThreadPtr)
				{
					FStatMessage* ThreadResult = ThisFrameMetaDataPerThreadPtr->Find(LongName);
					if (!ThreadResult)
					{
						ThreadResult = &ThisFrameMetaDataPerThreadPtr->Add(LongName, Item);
						ThreadResult->NameAndInfo.SetField<EStatOperation>(EStatOperation::Set);
						ThreadResult->NameAndInfo.SetFlag(EStatMetaFlags::IsPackedCCAndDuration, true);
						ThreadResult->Clear();
					}
				}

				FTimeInfo& ItemTime = Timing.FindOrAdd(LongName);

				if (Op == EStatOperation::ChildrenStart)
				{
					ItemTime.StartCalls++;
					ItemTime.Recursion++;
				}
				else
				{
					if (Op == EStatOperation::ChildrenEnd)
					{
						ItemTime.StopCalls++;
						ItemTime.Recursion--;
					}
					if (!ItemTime.Recursion) // doing aggregates here, so ignore misleading recursion which would be counted twice
					{
						FStatsUtils::AccumulateStat(*Result, Item, EStatOperation::Add);
						if (Depth && ThisFrameMetaDataPerThreadPtr)
						{
							FStatMessage& ThreadResult = ThisFrameMetaDataPerThreadPtr->FindChecked(LongName);
							FStatsUtils::AccumulateStat(ThreadResult, Item, EStatOperation::Add);
						}
					}

				}
			}
			else if( bAddNonStackStats )
			{
				FStatsUtils::AddNonStackStats( LongName, Item, Op, ThisFrameMetaData );
			}
		}
	}

	for (TMap<FName, FStatMessage>::TConstIterator It(ThisFrameMetaData); It; ++It)
	{
		OutStats.Add(It.Value());
	}
	
	if(OptionalOutThreadBreakdownMap)
	{
		for (TMap<FName, TMap<FName, FStatMessage>> ::TConstIterator ItThread(ThisFrameMetaDataPerThread); ItThread; ++ItThread)
		{
			const TMap<FName, FStatMessage>& ItemNameToMeta = ItThread.Value();
			const FName ThreadName = ItThread.Key();

			if(ItemNameToMeta.Num())
			{
				TArray<FStatMessage>& MetaForThread = OptionalOutThreadBreakdownMap->FindOrAdd(ThreadName);
				for (TMap<FName, FStatMessage>::TConstIterator It(ItemNameToMeta); It; ++It)
				{
					MetaForThread.Add(It.Value());
				}
			}
		}
	}
}

void FStatsThreadState::GetExclusiveAggregateStackStats( 
	int64 TargetFrame, 
	TArray<FStatMessage>& OutStats, 
	IItemFilter* Filter /*= nullptr*/, 
	bool bAddNonStackStats /*= true*/ ) const
{
	const TArray<FStatMessage>& CondensedMessages = GetCondensedHistory( TargetFrame );
	GetExclusiveAggregateStackStats( CondensedMessages, OutStats, Filter, bAddNonStackStats );
}

void FStatsThreadState::GetExclusiveAggregateStackStats( 
	const TArray<FStatMessage>& CondensedMessages, 
	TArray<FStatMessage>& OutStats, 
	IItemFilter* Filter /*= nullptr*/, 
	bool bAddNonStackStats /*= true */ ) const
{
	TMap<FName, FStatMessage> ThisFrameMetaData;
	TArray<FStatMessage> ChildDurationStack;

	for (int32 Index = 0; Index < CondensedMessages.Num(); Index++)
	{
		FStatMessage const& Item = CondensedMessages[Index];
		FName LongName = Item.NameAndInfo.GetRawName();

		EStatOperation::Type Op = Item.NameAndInfo.GetField<EStatOperation>();
		if ((Op == EStatOperation::ChildrenStart || Op == EStatOperation::ChildrenEnd ||  Op == EStatOperation::Leaf) && Item.NameAndInfo.GetFlag(EStatMetaFlags::IsCycle))
		{
			FStatMessage* Result = ThisFrameMetaData.Find(LongName);
			if (!Result)
			{
				Result = &ThisFrameMetaData.Add(LongName, Item);
				Result->NameAndInfo.SetField<EStatOperation>(EStatOperation::Set);
				Result->NameAndInfo.SetFlag(EStatMetaFlags::IsPackedCCAndDuration, true);
				Result->Clear();
			}
			if (Op == EStatOperation::ChildrenStart)
			{
				ChildDurationStack.Add(FStatMessage(Item));
			}
			else 
			{
				if (Op == EStatOperation::ChildrenEnd)
				{
					FStatsUtils::AccumulateStat(*Result, ChildDurationStack.Pop(), EStatOperation::Add);
				}
				else
				{
					FStatsUtils::AccumulateStat(*Result, Item, EStatOperation::Add);
				}
				if (ChildDurationStack.Num())
				{
					FStatsUtils::AccumulateStat(ChildDurationStack.Top(), Item, EStatOperation::Subtract, true);
				}
			}
		}
		else if( bAddNonStackStats )
		{
			FStatsUtils::AddNonStackStats( LongName, Item, Op, ThisFrameMetaData );
		}
	}

	for (TMap<FName, FStatMessage>::TConstIterator It(ThisFrameMetaData); It; ++It)
	{
		if (!Filter || Filter->Keep(It.Value()))
		{
			OutStats.Add(It.Value());
		}
	}
}

TArray<FStatMessage> const& FStatsThreadState::GetCondensedHistory( int64 TargetFrame ) const
{
	check(IsFrameValid(TargetFrame));

	TArray<FStatMessage> const* Result = CondensedStackHistory.FindRef(TargetFrame);
	if (Result)
	{
		return *Result;
	}
	SCOPE_CYCLE_COUNTER(STAT_StatsCondense);
	TArray<FStatMessage>& OutStats = *CondensedStackHistory.Add(TargetFrame, new TArray<FStatMessage>);
	Condense(TargetFrame, OutStats);
	INC_DWORD_STAT_BY(STAT_StatFramePacketsCondensed, OutStats.Num());
	return OutStats;
}

void FStatsThreadState::GetRawStackStats(int64 TargetFrame, FRawStatStackNode& Root, TArray<FStatMessage>* OutNonStackStats) const
{
	const FStatPacketArray& Frame = GetStatPacketArray( TargetFrame );
	TMap<FName, FStatMessage> ThisFrameNonStackStats;

	for (int32 PacketIndex = 0; PacketIndex < Frame.Packets.Num(); PacketIndex++)
	{
		FStatPacket const& Packet = *Frame.Packets[PacketIndex];
		const FName ThreadName = GetStatThreadName( Packet );

		FRawStatStackNode* ThreadRoot = Root.Children.FindRef(ThreadName);
		if (!ThreadRoot)
		{
			FString ThreadIdName = FStatsUtils::BuildUniqueThreadName( Packet.ThreadId );
			ThreadRoot = Root.Children.Add( ThreadName, new FRawStatStackNode( FStatMessage( ThreadName, EStatDataType::ST_int64, STAT_GROUP_TO_FStatGroup( STATGROUP_Threads )::GetGroupName(), STAT_GROUP_TO_FStatGroup( STATGROUP_Threads )::GetGroupCategory(), *ThreadIdName, true, true, false ) ) );
			ThreadRoot->Meta.NameAndInfo.SetFlag(EStatMetaFlags::IsPackedCCAndDuration, true);
			ThreadRoot->Meta.Clear();
		}

		{
			TArray<FStatMessage const*> StartStack;
			TArray<FRawStatStackNode*> Stack;
			Stack.Add(ThreadRoot);
			FRawStatStackNode* Current = Stack.Last();

			for (FStatMessage const& Item : Packet.StatMessages)
			{
				check(Item.NameAndInfo.GetFlag(EStatMetaFlags::DummyAlwaysOne));  // we should never be sending short names to the stats anymore

				EStatOperation::Type Op = Item.NameAndInfo.GetField<EStatOperation>();
				const FName LongName = Item.NameAndInfo.GetRawName();
				if (Op == EStatOperation::CycleScopeStart || Op == EStatOperation::CycleScopeEnd)
				{				
					check(Item.NameAndInfo.GetFlag(EStatMetaFlags::IsCycle));
					if (Op == EStatOperation::CycleScopeStart)
					{
						FRawStatStackNode* Result = Current->Children.FindRef(LongName);
						if (!Result)
						{
							Result = Current->Children.Add(LongName, new FRawStatStackNode(Item));
							Result->Meta.NameAndInfo.SetField<EStatOperation>(EStatOperation::Set);
							Result->Meta.NameAndInfo.SetFlag(EStatMetaFlags::IsPackedCCAndDuration, true);
							Result->Meta.Clear();
						}
						Stack.Add(Result);
						StartStack.Add(&Item);
						Current = Result;
					}
					if (Op == EStatOperation::CycleScopeEnd)
					{
						FStatMessage RootCall = FStatsUtils::ComputeCall(*StartStack.Pop(), Item);
						FStatsUtils::AccumulateStat(Current->Meta, RootCall, EStatOperation::Add);
						check(Current->Meta.NameAndInfo.GetFlag(EStatMetaFlags::IsPackedCCAndDuration));
						verify(Current == Stack.Pop());
						Current = Stack.Last();
					}
				}
				// We are using here EStatOperation::SpecialMessageMarker to indicate custom stat messages
				// At this moment only these messages are supported:
				//	EventWaitWithId
				//	EventTriggerWithId
				//	StatMarker
				else if( Op == EStatOperation::SpecialMessageMarker )
				{
					//const FName EncName = Item.NameAndInfo.GetEncodedName();
					const FName RawName = Item.NameAndInfo.GetRawName();

					const uint64 PacketEventIdAndCycles = Item.GetValue_Ptr();
					const uint32 EventId = uint32(PacketEventIdAndCycles >> 32);
					const uint32 EventCycles = uint32(PacketEventIdAndCycles & MAX_uint32);

					if (RawName == FStatConstants::RAW_EventWaitWithId || RawName==FStatConstants::RAW_EventTriggerWithId)
					{
						if (FStatConstants::RAW_EventWaitWithId == RawName)
						{
							TArray<FStatNameAndInfo> EventWaitStack;
							for (const auto& It : Stack)
							{
								EventWaitStack.Add( It->Meta.NameAndInfo );
							}

#if	UE_BUILD_DEBUG
							// Debug check, detect duplicates.
							FEventData* EventPtr = EventsHistory.Find( EventId );
							if (EventPtr && EventPtr->WaitStackStats.Num() > 0)
							{
								int32 k = 0; k++;
							}
#endif // UE_BUILD_DEBUG

							FEventData& EventStats = EventsHistory.FindOrAdd( EventId );
							EventStats.WaitStackStats = EventWaitStack;
							EventStats.Frame = EventStats.HasValidStacks() ? TargetFrame : 0; // Only to maintain history.
						}

						if (FStatConstants::RAW_EventTriggerWithId == RawName)
						{
							TArray<FStatNameAndInfo> EventTriggerStack;
							for (const auto& It : Stack)
							{
								EventTriggerStack.Add( It->Meta.NameAndInfo );
							}

#if	UE_BUILD_DEBUG
							// Debug check, detect duplicates.
							FEventData* EventPtr = EventsHistory.Find( EventId );
							if (EventPtr && EventPtr->TriggerStackStats.Num() > 0)
							{
								int32 k = 0; k++;
							}
#endif // UE_BUILD_DEBUG

							FEventData& EventStats = EventsHistory.FindOrAdd( EventId );

							EventStats.TriggerStackStats = EventTriggerStack;
							EventStats.Duration = EventCycles;
							EventStats.DurationMS = FPlatformTime::ToMilliseconds( EventCycles );
							EventStats.Frame = EventStats.HasValidStacks() ? TargetFrame : 0; // Only to maintain history.
						}
					}
					else if (RawName == FStatConstants::RAW_NamedMarker)
					{

					}
				}
				else if (OutNonStackStats)
				{
					FStatsUtils::AddNonStackStats( LongName, Item, Op, ThisFrameNonStackStats );
				}
			}
			// not true with partial frames check(Stack.Num() == 1 && Stack.Last() == ThreadRoot && Current == ThreadRoot);
		}
	}
	//add up the thread totals
	for (TMap<FName, FRawStatStackNode*>::TConstIterator ItRoot(Root.Children); ItRoot; ++ItRoot)
	{
		FRawStatStackNode* ThreadRoot = ItRoot.Value();
		for (TMap<FName, FRawStatStackNode*>::TConstIterator It(ThreadRoot->Children); It; ++It)
		{
			ThreadRoot->Meta.GetValue_int64() += It.Value()->Meta.GetValue_int64();
		}
	}
	if (OutNonStackStats)
	{
		for (TMap<FName, FStatMessage>::TConstIterator It(ThisFrameNonStackStats); It; ++It)
		{
			OutNonStackStats->Add(It.Value());
		}
	}
}

void FStatsThreadState::UncondenseStackStats(
	int64 TargetFrame, 
	FRawStatStackNode& Root, 
	IItemFilter* Filter /*= nullptr*/, 
	TArray<FStatMessage>* OutNonStackStats /*= nullptr */) const
{
	const TArray<FStatMessage>& CondensedMessages = GetCondensedHistory( TargetFrame );
	UncondenseStackStats( CondensedMessages, Root, Filter, OutNonStackStats );
}

void FStatsThreadState::UncondenseStackStats( 
	const TArray<FStatMessage>& CondensedMessages, 
	FRawStatStackNode& Root, 
	IItemFilter* Filter /*= nullptr*/, 
	TArray<FStatMessage>* OutNonStackStats /*= nullptr */ ) const
{
	TMap<FName, FStatMessage> ThisFrameNonStackStats;

	{
		TArray<FRawStatStackNode*> Stack;
		Stack.Add(&Root);
		FRawStatStackNode* Current = Stack.Last();

		for (int32 Index = 0; Index < CondensedMessages.Num(); Index++)
		{
			FStatMessage const& Item = CondensedMessages[Index];
			if (!Filter || Filter->Keep(Item))
			{
				EStatOperation::Type Op = Item.NameAndInfo.GetField<EStatOperation>();
				FName LongName = Item.NameAndInfo.GetRawName();
				if (Op == EStatOperation::ChildrenStart || Op == EStatOperation::ChildrenEnd || Op == EStatOperation::Leaf)
				{
					if (LongName != FStatConstants::NAME_ThreadRoot)
					{
						if (Op == EStatOperation::ChildrenStart || Op == EStatOperation::Leaf)
						{
							FRawStatStackNode* Result = Current->Children.FindRef(LongName);
							if (!Result)
							{
								Result = Current->Children.Add(LongName, new FRawStatStackNode(Item));
								Result->Meta.NameAndInfo.SetField<EStatOperation>(EStatOperation::Set);
							}
							else
							{
								FStatsUtils::AccumulateStat(Result->Meta, Item, EStatOperation::Add);
							}
							if (Op == EStatOperation::ChildrenStart)
							{
								Stack.Add(Result);
								Current = Result;
							}
						}
						if (Op == EStatOperation::ChildrenEnd)
						{
							verify(Current == Stack.Pop());
							Current = Stack.Last();
						}
					}
				}
				else if (OutNonStackStats)
				{
					FStatsUtils::AddNonStackStats( LongName, Item, Op, ThisFrameNonStackStats );
				}
			}
		}
	}
	if (OutNonStackStats)
	{
		for (TMap<FName, FStatMessage>::TConstIterator It(ThisFrameNonStackStats); It; ++It)
		{
			OutNonStackStats->Add(It.Value());
		}
	}
}

int64 FStatsThreadState::GetFastThreadFrameTimeInternal( int64 TargetFrame, int32 ThreadID, EThreadType::Type Thread ) const
{
	int64 Result = 0;

	const FStatPacketArray& Frame = GetStatPacketArray( TargetFrame );

	for( int32 PacketIndex = 0; PacketIndex < Frame.Packets.Num(); PacketIndex++ )
	{
		FStatPacket const& Packet = *Frame.Packets[PacketIndex];

		if( Packet.ThreadId == ThreadID || Packet.ThreadType == Thread )
		{
			const FStatMessagesArray& Data = Packet.StatMessages;
			for (FStatMessage const& Item : Data)
			{
				EStatOperation::Type Op = Item.NameAndInfo.GetField<EStatOperation>();
				FName LongName = Item.NameAndInfo.GetRawName();
				if (Op == EStatOperation::CycleScopeStart)
				{
					check(Item.NameAndInfo.GetFlag(EStatMetaFlags::IsCycle));
					Result -= Item.GetValue_int64();
					break;
				}
			}
			for (int32 Index = Data.Num() - 1; Index >= 0; Index--)
			{
				FStatMessage const& Item = Data[Index];
				EStatOperation::Type Op = Item.NameAndInfo.GetField<EStatOperation>();
				FName LongName = Item.NameAndInfo.GetRawName();
				if (Op == EStatOperation::CycleScopeEnd)
				{

					check(Item.NameAndInfo.GetFlag(EStatMetaFlags::IsCycle));
					Result += Item.GetValue_int64();
					break;
				}
			}
		}
	}
	return Result;
}

int64 FStatsThreadState::GetFastThreadFrameTime(int64 TargetFrame, EThreadType::Type Thread) const
{
	return GetFastThreadFrameTimeInternal( TargetFrame, 0, Thread );
}

int64 FStatsThreadState::GetFastThreadFrameTime( int64 TargetFrame, uint32 ThreadID ) const
{
	return GetFastThreadFrameTimeInternal( TargetFrame, ThreadID, EThreadType::Invalid );
}

FName FStatsThreadState::GetStatThreadName( const FStatPacket& Packet ) const
{
	FName ThreadName;
	if( Packet.ThreadType == EThreadType::Game )
	{
		ThreadName = NAME_GameThread;
	}
	else if( Packet.ThreadType == EThreadType::Renderer )
	{
		ThreadName = NAME_RenderThread;
	}
	else if ( Packet.ThreadType == EThreadType::EndOfPipe )
	{
		static const FName NAME_EndOfPipe = "EndOfPipe";
		ThreadName = NAME_EndOfPipe;
	}
	else if( Packet.ThreadType == EThreadType::Other )
	{
		TMap<uint32, FName>& MutableThreads = const_cast<TMap<uint32, FName>&>(Threads);
		FName& NewThreadName = MutableThreads.FindOrAdd( Packet.ThreadId );
		if( NewThreadName == NAME_None )
		{
			UE_LOGF( LogStats, Display, "There is no thread with id: %u. Please add thread metadata for this thread.", Packet.ThreadId );

			static const FName NAME_UnknownThread = "UnknownThread";
			NewThreadName = FName( *FStatsUtils::BuildUniqueThreadName( Packet.ThreadId ) );

			UE_AUTORTFM_OPEN
			{
				// This is an unknown thread, but still we need the metadata in the system.
				FStartupMessages::Get().AddThreadMetadata( NAME_UnknownThread, Packet.ThreadId );
			};
		}
		ThreadName = NewThreadName;
	}

	check( ThreadName != NAME_None );
	return ThreadName;
}

void FStatsThreadState::Condense(int64 TargetFrame, TArray<FStatMessage>& OutStats) const
{
	OutStats.Emplace(FStatConstants::AdvanceFrame.GetEncodedName(), EStatOperation::AdvanceFrameEventGameThread, TargetFrame, false);
	OutStats.Emplace(FStatConstants::AdvanceFrame.GetEncodedName(), EStatOperation::AdvanceFrameEventRenderThread, TargetFrame, false);
	OutStats.Emplace(FStatConstants::AdvanceFrame.GetEncodedName(), EStatOperation::AdvanceFrameEventEndOfPipe, TargetFrame, false);
	FRawStatStackNode Root;
	GetRawStackStats(TargetFrame, Root, &OutStats);
	TArray<FStatMessage> StackStats;
	Root.Encode(StackStats);
	OutStats += StackStats;
}

void FStatsThreadState::FindOrAddMetaData(FStatMessage const& Item)
{
	const FName ShortName = Item.NameAndInfo.GetShortName();
	const FName GroupName = Item.NameAndInfo.GetGroupName();

	// Whether to add to the threads group.
	const bool bIsThread = FStatConstants::NAME_ThreadGroup == GroupName;
	if (bIsThread)
	{
		// The description of a thread group contains the thread id
		const FString Desc = Item.NameAndInfo.GetDescription();
		Threads.Add(FStatsUtils::ParseThreadID(Desc), ShortName);
	}

	FStatMessage* Result = ShortNameToLongName.Find(ShortName);
	if (!Result)
	{
		const FName LongName = Item.NameAndInfo.GetRawName();

		check(ShortName != LongName);
		FStatMessage AsSet(Item);
		AsSet.Clear();

		ShortNameToLongName.Add(ShortName, AsSet); // we want this to be a clear, but it should be a SetLongName
		AsSet.NameAndInfo.SetField<EStatOperation>(EStatOperation::Set);
		check(Item.NameAndInfo.GetField<EStatMetaFlags>());
		Groups.Add(GroupName, ShortName);
		if (GroupName != NAME_Groups && !Item.NameAndInfo.GetFlag(EStatMetaFlags::ShouldClearEveryFrame))
		{
			NotClearedEveryFrame.Add(LongName, AsSet);
		}
		if (Item.NameAndInfo.GetFlag(EStatMetaFlags::IsMemory) && ShortName.ToString().StartsWith(TEXT("MCR_")))
		{
			// this is a pool size
			FPlatformMemory::EMemoryCounterRegion Region = FPlatformMemory::EMemoryCounterRegion(Item.NameAndInfo.GetField<EMemoryRegion>());
			if (MemoryPoolToCapacityLongName.Contains(Region))
			{
				UE_LOGF(LogStats, Warning, "MetaData mismatch. Did you assign a memory pool capacity two different ways? %ls vs %ls", *LongName.ToString(), *MemoryPoolToCapacityLongName[Region].ToString());
			}
			else
			{
				MemoryPoolToCapacityLongName.Add(Region, LongName);
			}
		}

		// Add the info to the task graph so we can inform the game thread
		DECLARE_CYCLE_STAT(TEXT("FSimpleDelegateGraphTask.StatsGroupToGame"),
			STAT_FSimpleDelegateGraphTask_StatsGroupToGame,
			STATGROUP_TaskGraphTasks);

		FSimpleDelegateGraphTask::CreateAndDispatchWhenReady
		(
			FSimpleDelegateGraphTask::FDelegate::CreateRaw(&FStatGroupGameThreadNotifier::Get(), &FStatGroupGameThreadNotifier::NewData, Item.NameAndInfo),
			GET_STATID(STAT_FSimpleDelegateGraphTask_StatsGroupToGame), nullptr, ENamedThreads::GameThread
		);
	}
}

void FStatsThreadState::AddMissingStats(TArray<FStatMessage>& Dest, TSet<FName> const& EnabledItems) const
{
	TSet<FName> NamesToTry(EnabledItems);
	TMap<FName, int32> NameToIndex;
	for (int32 Index = 0; Index < Dest.Num(); Index++)
	{
		NamesToTry.Remove(Dest[Index].NameAndInfo.GetShortName());
	}

	for (auto It = NamesToTry.CreateConstIterator(); It; ++It)
	{
		FStatMessage const* Zero = ShortNameToLongName.Find(*It);
		if (Zero)
		{
			Dest.Add(*Zero);
		}
	}
}

void FStatsThreadState::FindAndDumpMemoryExtensiveStats( const FStatPacketArray& Frame )
{
	int32 TotalMessages = 0;
	TMap<FName,int32> NameToCount;

	// Generate some data statistics.
	for( const FStatPacket* StatPacket : Frame.Packets )
	{
		for( const FStatMessage& Message : StatPacket->StatMessages )
		{
			const FName ShortName = Message.NameAndInfo.GetShortName();
			NameToCount.FindOrAdd(ShortName) += 1;
			TotalMessages++;
		}
	}

	// Dump stats to the log.
	NameToCount.ValueSort( TGreater<uint32>() );

	const float MaxPctDisplayed = 0.9f;
	int32 CurrentIndex = 0;
	int32 DisplayedSoFar = 0;
	UE_LOGF( LogStats, Warning, "%2ls, %32ls, %5ls", TEXT("No"), TEXT("Name"), TEXT("Count") );
	for( const auto& It : NameToCount )
	{
		UE_LOGF( LogStats, Warning, "%2i, %32ls, %5i", CurrentIndex, *It.Key.ToString(), It.Value );
		CurrentIndex++;
		DisplayedSoFar += It.Value;

		const float CurrentPct = (float)DisplayedSoFar/(float)TotalMessages;
		if( CurrentPct > MaxPctDisplayed )
		{
			break;
		}
	}
}

void FStatsUtils::GetNameAndGroup(FStatMessage const& Item, FString& OutName, FString& OutGroup)
{
	const FString ShortName = Item.NameAndInfo.GetShortName().ToString();
	const FName Group = Item.NameAndInfo.GetGroupName();
	const FName Category = Item.NameAndInfo.GetGroupCategory();
	OutName = Item.NameAndInfo.GetDescription();
	OutName.TrimStartInline();

	if (OutName != ShortName)
	{
		if (OutName.Len())
		{
			OutName += TEXT(" - ");
		}
		OutName += ShortName;
	}

	if (Group != NAME_None)
	{
		OutGroup = TEXT(" - ");
		OutGroup += *Group.ToString();
	}
	if (Category != NAME_None)
	{
		OutGroup += TEXT(" - ");
		OutGroup += *Category.ToString();
	}
}

FString FStatsUtils::DebugPrint(FStatMessage const& Item)
{
	FString Result("Invalid");
	switch (Item.NameAndInfo.GetField<EStatDataType>())
	{
	case EStatDataType::ST_int64:
		if (Item.NameAndInfo.GetFlag(EStatMetaFlags::IsPackedCCAndDuration))
		{
			Result = FString::Printf(TEXT("%.3fms (%4d)"), FPlatformTime::ToMilliseconds(FromPackedCallCountDuration_Duration(Item.GetValue_int64())), FromPackedCallCountDuration_CallCount(Item.GetValue_int64()));
		}
		else if (Item.NameAndInfo.GetFlag(EStatMetaFlags::IsCycle))
		{
			Result = FString::Printf(TEXT("%.3fms"), FPlatformTime::ToMilliseconds64(Item.GetValue_int64()));
		}
		else if (Item.NameAndInfo.GetFlag(EStatMetaFlags::IsMemory))
		{
			Result = FString::Printf(TEXT("%.3fMB"), (double)Item.GetValue_int64() / 1024.0 / 1024.0);
		}
		else
		{
			Result = FString::Printf(TEXT("%llu"), Item.GetValue_int64());
		}
		break;
	case EStatDataType::ST_double:
		Result = FString::Printf(TEXT("%.1f"), Item.GetValue_double());
		break;
	case EStatDataType::ST_FName:
		Result = Item.GetValue_FName().ToString();
		break;
	}

	Result = FString(FCString::Spc(FMath::Max<int32>(0, 14 - Result.Len()))) + Result;

	FString Desc, GroupAndCategory;

	GetNameAndGroup(Item, Desc, GroupAndCategory);

	return FString::Printf(TEXT("  %s  -  %s%s"), *Result, *Desc, *GroupAndCategory);
}

void FStatsUtils::AddMergeStatArray(TArray<FStatMessage>& Dest, TArray<FStatMessage> const& Src)
{
	TMap<FName, int32> NameToIndex;
	for (int32 Index = 0; Index < Dest.Num(); Index++)
	{
		NameToIndex.Add(Dest[Index].NameAndInfo.GetRawName(), Index);
	}
	for (int32 Index = 0; Index < Src.Num(); Index++)
	{
		int32* DestIndexPtr = NameToIndex.Find(Src[Index].NameAndInfo.GetRawName());
		int32 DestIndex = -1;
		if (DestIndexPtr)
		{
			DestIndex = *DestIndexPtr;
		}
		else
		{
			DestIndex = Dest.Num();
			NameToIndex.Add(Src[Index].NameAndInfo.GetRawName(), DestIndex);
			FStatMessage NewMessage(Src[Index]);
			NewMessage.Clear();
			Dest.Add(NewMessage);
		}
		AccumulateStat(Dest[DestIndex], Src[Index], EStatOperation::Add);
	}
}

void FStatsUtils::MaxMergeStatArray(TArray<FStatMessage>& Dest, TArray<FStatMessage> const& Src)
{
	TMap<FName, int32> NameToIndex;
	for (int32 Index = 0; Index < Dest.Num(); Index++)
	{
		NameToIndex.Add(Dest[Index].NameAndInfo.GetRawName(), Index);
	}
	for (int32 Index = 0; Index < Src.Num(); Index++)
	{
		int32* DestIndexPtr = NameToIndex.Find(Src[Index].NameAndInfo.GetRawName());
		int32 DestIndex = -1;
		if (DestIndexPtr)
		{
			DestIndex = *DestIndexPtr;
		}
		else
		{
			DestIndex = Dest.Num();
			NameToIndex.Add(Src[Index].NameAndInfo.GetRawName(), DestIndex);
			FStatMessage NewMessage(Src[Index]);
			NewMessage.Clear();
			Dest.Add(NewMessage);
		}
		AccumulateStat(Dest[DestIndex], Src[Index], EStatOperation::MaxVal);
	}
}

void FStatsUtils::DivideStat(FStatMessage& Dest, uint32 Div)
{
	switch (Dest.NameAndInfo.GetField<EStatDataType>())
	{
	case EStatDataType::ST_int64:
		if (Dest.NameAndInfo.GetFlag(EStatMetaFlags::IsPackedCCAndDuration))
		{
			Dest.GetValue_int64() = ToPackedCallCountDuration(
				(FromPackedCallCountDuration_CallCount(Dest.GetValue_int64()) + (Div >> 1)) / Div,
				(FromPackedCallCountDuration_Duration(Dest.GetValue_int64()) + (Div >> 1)) / Div);
		}
		else if (Dest.NameAndInfo.GetFlag(EStatMetaFlags::IsCycle))
		{
			Dest.GetValue_int64() = (Dest.GetValue_int64() + Div - 1) / Div;
		}
		else
		{
			int64 Val = Dest.GetValue_int64();
			Dest.NameAndInfo.SetField<EStatDataType>(EStatDataType::ST_double);
			Dest.GetValue_double() = (double)(Val) / (double)Div;
		}
		break;
	case EStatDataType::ST_double:
		Dest.GetValue_double() /= (double)Div;
		break;
	default:
		check(0);
	}
}

void FStatsUtils::DivideStatArray(TArray<FStatMessage>& DestArray, uint32 Div)
{
	for (int32 Index = 0; Index < DestArray.Num(); Index++)
	{
		FStatMessage &Dest = DestArray[Index];
		DivideStat(Dest, Div);
	}
}

void FStatsUtils::AccumulateStat(FStatMessage& Dest, FStatMessage const& Item, EStatOperation::Type Op /*= EStatOperation::Invalid*/, bool bAllowNameMismatch /*= false*/)
{
	check(bAllowNameMismatch || Dest.NameAndInfo.GetRawName() == Item.NameAndInfo.GetRawName());

	if (Op == EStatOperation::Invalid)
	{
		Op = Item.NameAndInfo.GetField<EStatOperation>();
	}
	check(Dest.NameAndInfo.GetField<EStatDataType>() == Item.NameAndInfo.GetField<EStatDataType>());
	check(Dest.NameAndInfo.GetFlag(EStatMetaFlags::IsPackedCCAndDuration) == Item.NameAndInfo.GetFlag(EStatMetaFlags::IsPackedCCAndDuration));
	switch (Item.NameAndInfo.GetField<EStatDataType>())
	{
		case EStatDataType::ST_int64:
			switch (Op)
			{
				case EStatOperation::Set:
					Dest.GetValue_int64() = Item.GetValue_int64();
					break;
				case EStatOperation::Clear:
					Dest.GetValue_int64() = 0;
					break;
				case EStatOperation::Add:
					Dest.GetValue_int64() += Item.GetValue_int64();
					break;
				case EStatOperation::Subtract:
					if (Dest.NameAndInfo.GetFlag(EStatMetaFlags::IsPackedCCAndDuration))
					{
						// we don't subtract call counts, only times
						Dest.GetValue_int64() = ToPackedCallCountDuration(
							FromPackedCallCountDuration_CallCount(Dest.GetValue_int64()),
							FromPackedCallCountDuration_Duration(Dest.GetValue_int64()) - FromPackedCallCountDuration_Duration(Item.GetValue_int64()));
					}
					else
					{
						Dest.GetValue_int64() -= Item.GetValue_int64();
					}
					break;
				case EStatOperation::MaxVal:
					StatOpMaxVal_Int64( Dest.NameAndInfo, Dest.GetValue_int64(), Item.GetValue_int64() );
					break;
				default:
					check(0);
			}
			break;

		case EStatDataType::ST_double:
			switch (Op)
			{
				case EStatOperation::Set:
					Dest.GetValue_double() = Item.GetValue_double();
					break;
				case EStatOperation::Clear:
					Dest.GetValue_double() = 0;
					break;
				case EStatOperation::Add:
					Dest.GetValue_double() += Item.GetValue_double();
					break;
				case EStatOperation::Subtract:
					Dest.GetValue_double() -= Item.GetValue_double();
					break;
				case EStatOperation::MaxVal:
					Dest.GetValue_double() = FMath::Max<double>(Dest.GetValue_double(), Item.GetValue_double());
					break;
				default:
					check(0);
			}
			break;

		// Nothing here at this moment.
		case EStatDataType::ST_Ptr:
			break;

		default:
			check(0);
	}
}

FString FStatsUtils::FromEscapedString(const TCHAR* Escaped)
{
	FString Result;
	FString Input(Escaped);
	while (Input.Len())
	{
		{
			int32 Index = Input.Find(TEXT("$"), ESearchCase::CaseSensitive);
			if (Index == INDEX_NONE)
			{
				Result += Input;
				break;
			}
			Result += Input.Left(Index);
			Input.RightChopInline(Index + 1, EAllowShrinking::No);

		}
		{
			int32 IndexEnd = Input.Find(TEXT("$"), ESearchCase::CaseSensitive);
			if (IndexEnd == INDEX_NONE)
			{
				checkStats(0); // malformed escaped fname
				Result += Input;
				break;
			}
			FString Number = Input.Left(IndexEnd);
			Input.RightChopInline(IndexEnd + 1, EAllowShrinking::No);
			Result.AppendChar(TCHAR(uint32(FCString::Atoi64(*Number))));
		}
	}
	return Result;
}

void FStatsUtils::ToEscapedString(FStringView Input, FStringBuilderBase& Output)
{
	constexpr FAsciiSet InvalidChars = FAsciiSet(INVALID_NAME_CHARACTERS) + '$';
	while (true)
	{
		FStringView ValidInput = FAsciiSet::FindPrefixWithout(Input, InvalidChars);
		Output.Append(ValidInput);

		if (ValidInput.end() == Input.end())
		{
			break;
		}

		// Replace the invalid character with a special string
		Output << '$' << static_cast<uint32>(*ValidInput.end()) << '$';

		Input.RemovePrefix(ValidInput.Len() + 1);
	}
}


void FComplexStatUtils::AddAndMinMax( FComplexStatMessage& Dest, const FStatMessage& Item, EComplexStatField::Type SumIndex, EComplexStatField::Type MaxIndex, EComplexStatField::Type MinIndex )
{
	check(Dest.NameAndInfo.GetRawName() == Item.NameAndInfo.GetRawName());

	// Copy the data type from the other stack node.
	if( Dest.NameAndInfo.GetField<EStatDataType>() == EStatDataType::ST_None )
	{
		const EStatDataType::Type ItemStatDataType = Item.NameAndInfo.GetField<EStatDataType>();
		Dest.NameAndInfo.SetField<EStatDataType>(ItemStatDataType);
		if( ItemStatDataType == EStatDataType::ST_int64 )
		{
			Dest.GetValue_int64(MinIndex) = INT64_MAX; 
		}
		else if( ItemStatDataType == EStatDataType::ST_double )
		{
			Dest.GetValue_double(MinIndex) = DBL_MAX;
		}
	}

	const EStatDataType::Type StatDataType = Dest.NameAndInfo.GetField<EStatDataType>();

	// Total time.
	if( StatDataType != EStatDataType::ST_None && StatDataType != EStatDataType::ST_FName )
	{
		if( StatDataType == EStatDataType::ST_int64 )
		{
			Dest.GetValue_int64(SumIndex) += Item.GetValue_int64();
		}
		else if( StatDataType == EStatDataType::ST_double )
		{
			Dest.GetValue_double(SumIndex) += Item.GetValue_double();
		}
	}

	// Maximum/Minimum time.
	if( StatDataType != EStatDataType::ST_None && StatDataType != EStatDataType::ST_FName)
	{
		if( StatDataType == EStatDataType::ST_int64 )
		{
			FStatsUtils::StatOpMaxVal_Int64( Dest.NameAndInfo, Dest.GetValue_int64(MaxIndex), Item.GetValue_int64() );
			FStatsUtils::StatOpMinVal_Int64( Dest.NameAndInfo, Dest.GetValue_int64(MinIndex), Item.GetValue_int64() );
		}
		else if( StatDataType == EStatDataType::ST_double )
		{
			Dest.GetValue_double(MaxIndex) = FMath::Max<double>(Dest.GetValue_double(MaxIndex), Item.GetValue_double());
			Dest.GetValue_double(MinIndex) = FMath::Min<double>(Dest.GetValue_double(MinIndex), Item.GetValue_double());
		}
	}
}

void FComplexStatUtils::DivideStat( FComplexStatMessage& Dest, uint32 Div, EComplexStatField::Type SumIndex, EComplexStatField::Type DestIndex )
{
	switch (Dest.NameAndInfo.GetField<EStatDataType>())
	{
	case EStatDataType::ST_int64:
		if (Dest.NameAndInfo.GetFlag(EStatMetaFlags::IsPackedCCAndDuration))
		{
			const int64 PackedCCAndDuration = ToPackedCallCountDuration(
				(FromPackedCallCountDuration_CallCount(Dest.GetValue_int64(SumIndex)) + (Div >> 1)) / Div,
				(FromPackedCallCountDuration_Duration(Dest.GetValue_int64(SumIndex)) + (Div >> 1)) / Div);

			Dest.GetValue_int64(DestIndex) = PackedCCAndDuration;
		}
		else if (Dest.NameAndInfo.GetFlag(EStatMetaFlags::IsCycle))
		{
			Dest.GetValue_int64(DestIndex) = (Dest.GetValue_int64(SumIndex) + Div - 1) / Div;
		}
		else
		{
			const int64 Val = Dest.GetValue_int64(SumIndex);

			// Stat data type has change, we need to convert remaining fields to the new data type.
			Dest.FixStatData( EStatDataType::ST_double );

			Dest.GetValue_double(DestIndex) = (double)(Val) / (double)Div;
		}
		break;
	case EStatDataType::ST_double:
		Dest.GetValue_double(DestIndex) = Dest.GetValue_double(SumIndex) / (double)Div;
		break;
	}
}

void FComplexStatUtils::MergeAddAndMinMaxArray( TArray<FComplexStatMessage>& Dest, const TArray<FStatMessage>& Source, EComplexStatField::Type SumIndex, EComplexStatField::Type MaxIndex, EComplexStatField::Type MinIndex )
{
	TMap<FName, int32> NameToIndex;
	for( int32 Index = 0; Index < Dest.Num(); Index++ )
	{
		const FName RawName = Dest[Index].NameAndInfo.GetRawName();
		NameToIndex.Add( RawName, Index );
	}

	for( int32 Index = 0; Index < Source.Num(); Index++ )
	{
		const int32& DestIndex = NameToIndex.FindChecked( Source[Index].NameAndInfo.GetRawName() );
		FComplexStatUtils::AddAndMinMax( Dest[DestIndex], Source[Index], SumIndex, MaxIndex, MinIndex );
	}
}

void FComplexStatUtils::DiviveStatArray( TArray<FComplexStatMessage>& Dest, uint32 Div, EComplexStatField::Type SumIndex, EComplexStatField::Type DestIndex )
{
	for( int32 Index = 0; Index < Dest.Num(); Index++ )
	{
		FComplexStatMessage &Aggregated = Dest[Index];
		FComplexStatUtils::DivideStat( Aggregated, Div, SumIndex, DestIndex );
	}
}

/** Broadcast the name and info data about any newly registered stat groups */
CORE_API void CheckForRegisteredStatGroups()
{
	FStatGroupGameThreadNotifier::Get().SendData();
}

/** Clear the data that's pending to be sent to prevent it accumulating when not claimed by a delegate */
CORE_API void ClearPendingStatGroups()
{
	FStatGroupGameThreadNotifier::Get().ClearData();
}

#endif
