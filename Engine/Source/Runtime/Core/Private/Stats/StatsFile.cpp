// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	StatsFile.cpp: Implements stats file related functionality.
=============================================================================*/

#include "Stats/StatsFile.h"

#if STATS && UE_ENABLE_STATS_FILE_DEPRECATED_IN_5_8

#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "Internationalization/Internationalization.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/MemoryReader.h"
#include "Templates/UniquePtr.h"

#include "Misc/ScopeExit.h"

DECLARE_CYCLE_STAT( TEXT( "Stream File" ), STAT_StreamFile, STATGROUP_StatSystem );
DECLARE_CYCLE_STAT( TEXT( "Wait For Write" ), STAT_StreamFileWaitForWrite, STATGROUP_StatSystem );

#define LOCTEXT_NAMESPACE "StatsFile"

/*-----------------------------------------------------------------------------
	FAsyncStatsWrite
-----------------------------------------------------------------------------*/

/**
*	Helper class used to save the capture stats data via the background thread.
*	!!CAUTION!! Can exist only one instance at the same time. Synchronized via EnsureCompletion.
*/
class FAsyncStatsWrite : public FNonAbandonableTask
{
public:
	/**
	 *	Pointer to the instance of the stats write file.
	 *	Generally speaking accessing this pointer by a different thread is not thread-safe.
	 *	But in this specific case it is.
	 *	@see SendTask
	 */
	IStatsWriteFile* Outer;

	/** Data for the file. Moved via Exchange. */
	TArray<uint8> Data;

	/** Constructor. */
	FAsyncStatsWrite( IStatsWriteFile* InStatsWriteFile )
		: Outer( InStatsWriteFile )
	{
		Exchange( Data, InStatsWriteFile->OutData );
	}

	/** Write compressed data to the file. */
	void DoWork()
	{
		check( Data.Num() );
		FArchive& Ar = *Outer->File;

		// Seek to the end of the file.
		const int64 TotalSize = Ar.TotalSize();
		// Update the file size.
		Outer->FileSize = TotalSize;
		Ar.Seek( TotalSize );
		const int64 FrameFileOffset = Ar.Tell();

		FCompressedStatsData CompressedData( Data, Outer->CompressedData );
		Ar << CompressedData;

		Outer->FinalizeSavingData( FrameFileOffset );
	}

	FORCEINLINE TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT( FAsyncStatsWrite, STATGROUP_ThreadPoolAsyncTasks );
	}
};

/*-----------------------------------------------------------------------------
	FStatsLoadedState
-----------------------------------------------------------------------------*/

void FStatsLoadedState::ProcessMetaDataAndLeaveDataOnly( TArray<FStatMessage>& CondensedMessages )
{
	for (int32 Index = 0; Index < CondensedMessages.Num(); Index++)
	{
		FStatMessage& Item = CondensedMessages[Index];
		EStatOperation::Type Op = Item.NameAndInfo.GetField<EStatOperation>();
		if (Op == EStatOperation::SetLongName)
		{
			FindOrAddMetaData( Item );
		}
		else if (Op == EStatOperation::AdvanceFrameEventGameThread)
		{
			// The following messages contain only the stats data.
			CondensedMessages.RemoveAt( 0, Index, EAllowShrinking::Yes );
			break;
		}
	}
}

void FStatsLoadedState::AddFrameFromCondensedMessages( TArray<FStatMessage>& CondensedMessages )
{
	// @see FStatsThreadState::Condense
	AdvanceFrameForLoad( CondensedMessages );

	const int64 MinFrameToKeep = CurrentGameFrame - HistoryFrames;

	if (!CondensedStackHistory.Contains( CurrentGameFrame ))
	{
		TArray<FStatMessage>* Save = new TArray<FStatMessage>();
		Exchange( *Save, CondensedMessages );

		for (auto It = CondensedStackHistory.CreateIterator(); It; ++It)
		{
			const int64 ThisFrame = It.Key();
			if (ThisFrame < MinFrameToKeep)
			{
				delete It.Value();
				It.RemoveCurrent();
			}
		}
		for (auto It = GoodFrames.CreateIterator(); It; ++It)
		{
			const int64 ThisFrame = *It;
			if (!CondensedStackHistory.Contains( ThisFrame ))
			{
				It.RemoveCurrent();
			}
		}

		CondensedStackHistory.Add( CurrentGameFrame, Save );
		GoodFrames.Add( CurrentGameFrame );
	}
}

int64 FStatsLoadedState::GetOldestValidFrame() const
{
	if (MaxFrameSeen < 0 || MinFrameSeen < 0)
	{
		return -1;
	}
	return MinFrameSeen;
}

int64 FStatsLoadedState::GetLatestValidFrame() const
{
	if (MaxFrameSeen < 0 || MinFrameSeen < 0)
	{
		return -1;
	}
	if (MaxFrameSeen > MinFrameSeen)
	{
		return MaxFrameSeen - 1;
	}
	return MaxFrameSeen;
}

void FStatsLoadedState::AdvanceFrameForLoad( TArray<FStatMessage>& CondensedMessages )
{
	// @see FStatsThreadState::Condense
	check( CondensedMessages.Num() >= 2 );
	const FStatMessage& AdvanceGameThread = CondensedMessages[0];

	// Sanity checks.
	check( AdvanceGameThread.NameAndInfo.GetField<EStatOperation>() == EStatOperation::AdvanceFrameEventGameThread );
	check( AdvanceGameThread.NameAndInfo.GetField<EStatDataType>() == EStatDataType::ST_int64 );
	check( AdvanceGameThread.GetValue_int64() > 0 );

	CurrentGameFrame = AdvanceGameThread.GetValue_int64();
	CurrentRenderFrame = CurrentGameFrame;
	CurrentEndOfPipeFrame = CurrentGameFrame;

	if (CurrentGameFrame > MaxFrameSeen)
	{
		MaxFrameSeen = CurrentGameFrame;
	}
	if (MinFrameSeen < 0)
	{
		MinFrameSeen = CurrentGameFrame;
	}
}

/*-----------------------------------------------------------------------------
	FStastsWriteStream
-----------------------------------------------------------------------------*/

void FStatsWriteStream::WriteMetadata( FArchive& Ar )
{
	const FStatsThreadState& Stats = FStatsThreadState::GetLocalState();
	for (const auto& It : Stats.ShortNameToLongName)
	{
		const FStatMessage& StatMessage = It.Value;
		WriteMessage( Ar, StatMessage );
	}
}

void FStatsWriteStream::WriteCondensedMessages( FArchive& Ar, int64 TargetFrame )
{
	const FStatsThreadState& Stats = FStatsThreadState::GetLocalState();
	const TArray<FStatMessage>& Data = Stats.GetCondensedHistory( TargetFrame );
	for (const FStatMessage& It : Data)
	{
		WriteMessage( Ar, It );
	}
}

/*-----------------------------------------------------------------------------
	IStatsWriteFile
-----------------------------------------------------------------------------*/

IStatsWriteFile::IStatsWriteFile()
	: File( nullptr )
	, AsyncTask( nullptr )
	, FileSize( 0 )
	, StartTime( FPlatformTime::Seconds() )
{
	// Reserve 1MB.
	CompressedData.Reserve( EStatsFileConstants::MAX_COMPRESSED_SIZE );
}

#define STATSFILE_TEMPORARY_FILENAME_SUFFIX		TEXT(".inprogress")

void IStatsWriteFile::Start( const FString& InFilename )
{
	const FString PathName = *(FPaths::ProfilingDir() + TEXT( "UnrealStats/" ));
	FString Filename = PathName + InFilename;

	if (FPaths::GetExtension(Filename).IsEmpty())
	{
		Filename += TEXT(".uestats");
	}

	const FString Path = FPaths::GetPath( Filename );
	IFileManager::Get().MakeDirectory( *Path, true );

	const FString TempFilename = Filename + STATSFILE_TEMPORARY_FILENAME_SUFFIX;

	UE_LOGF( LogStats, Log, "Opening stats file: %ls", *TempFilename );

	File = IFileManager::Get().CreateFileWriter( *TempFilename );
	if (!File)
	{
		UE_LOGF( LogStats, Error, "Could not open: %ls", *TempFilename );
	}
	else
	{
		ArchiveFilename = Filename;
		WriteHeader();
		SetDataDelegate( true );
		StatsPrimaryEnableAdd();
	}
}


void IStatsWriteFile::Stop()
{
	if (IsValid())
	{
		StatsPrimaryEnableSubtract();
		SetDataDelegate( false );
		SendTask();
		SendTask();
		Finalize();

		File->Close();
		delete File;
		File = nullptr;

		const FString TempFilename = ArchiveFilename + STATSFILE_TEMPORARY_FILENAME_SUFFIX;
		if (!IFileManager::Get().Move( *ArchiveFilename, *TempFilename ))
		{
			UE_LOGF( LogStats, Warning, "Could not rename stats file: %ls to final name %ls", *TempFilename, *ArchiveFilename );
		}

		UE_LOGF( LogStats, Log, "Wrote stats file: %ls", *ArchiveFilename );
		FCommandStatsFile::Get().LastFileSaved = ArchiveFilename;
	}
}

FText IStatsWriteFile::GetFileMetaDesc() const
{
	const FTimespan Duration = FTimespan( 0, 0, (int32)(FPlatformTime::Seconds() - StartTime) );

	const FText FileMetaDesc = FText::Format( LOCTEXT( "FileMetaDesc_Fmt", "STATS FILE: Duration: {0}, Filesize: {1}" ), FText::AsTimespan( Duration ), FText::AsMemory( (SIZE_T)FileSize ) );
	return FileMetaDesc;
}

#undef STATSFILE_TEMPORARY_FILENAME_SUFFIX

void IStatsWriteFile::WriteHeader()
{
	FMemoryWriter MemoryWriter( OutData, false, true );
	FArchive& Ar = File ? *File : MemoryWriter;

	uint32 Magic = EStatMagicWithHeader::MAGIC;
	// Serialize magic value.
	Ar << Magic;

	// Serialize dummy header, overwritten in Finalize.
	Header.Version = EStatMagicWithHeader::VERSION_LATEST;
	Header.PlatformName = FPlatformProperties::PlatformName();
	Ar << Header;

	// Serialize metadata.
	WriteMetadata( Ar );
	Ar.Flush();
}

void IStatsWriteFile::Finalize()
{
	FArchive& Ar = *File;

	// Write dummy compression size, so we can detect the end of the file.
	FCompressedStatsData::WriteEndOfCompressedData( Ar );

	// Real header, written at start of the file, but written out right before we close the file.

	// Write out frame table and update header with offset and count.
	Header.FrameTableOffset = Ar.Tell();
	// This is ok to access the frames info, the async write thread is dead.
	Ar << FramesInfo;

	const FStatsThreadState& Stats = FStatsThreadState::GetLocalState();

	// Add FNames from the stats metadata.
	for (const auto& It : Stats.ShortNameToLongName)
	{
		const FStatMessage& StatMessage = It.Value;
		FNamesSent.Add( StatMessage.NameAndInfo.GetRawName().GetComparisonIndex() );
	}

	// Create a copy of names.
	TSet<FNameEntryId> FNamesToSent = FNamesSent;
	FNamesSent.Empty( FNamesSent.Num() );

	// Serialize FNames.
	Header.FNameTableOffset = Ar.Tell();
	Header.NumFNames = FNamesToSent.Num();
	for (FNameEntryId Id : FNamesToSent)
	{
		WriteFName( Ar, FStatNameAndInfo( FName( Id, Id, 0 ), false ) );
	}

	// Serialize metadata messages.
	Header.MetadataMessagesOffset = Ar.Tell();
	Header.NumMetadataMessages = Stats.ShortNameToLongName.Num();
	WriteMetadata( Ar );

	// Seek to the position just after a magic value of the file and write out proper header.
	Ar.Seek( sizeof( uint32 ) );
	Ar << Header;
}

void IStatsWriteFile::WaitTask()
{
	if (AsyncTask)
	{
		SCOPE_CYCLE_COUNTER( STAT_StreamFileWaitForWrite );
		AsyncTask->EnsureCompletion();
		delete AsyncTask;
		AsyncTask = nullptr;
	}
}

void IStatsWriteFile::SendTask()
{
	WaitTask();

	if (OutData.Num())
	{
		AsyncTask = new FAsyncTask<FAsyncStatsWrite>( this );
		check( !OutData.Num() );
		//check( !ThreadCycles.Num() );
		AsyncTask->StartBackgroundTask();
	}
}

/*-----------------------------------------------------------------------------
	FStatsWriteFile
-----------------------------------------------------------------------------*/


void FStatsWriteFile::SetDataDelegate( bool bSet )
{
	FStatsThreadState const& Stats = FStatsThreadState::GetLocalState();
	if (bSet)
	{
		DataDelegateHandle = Stats.NewFrameDelegate.AddRaw( this, &FStatsWriteFile::WriteFrame );
	}
	else
	{
		Stats.NewFrameDelegate.Remove( DataDelegateHandle );
	}
}

void FStatsWriteFile::WriteFrame( int64 TargetFrame )
{
	// #Stats: 2015-06-17 Add stat startfile -num=number of frames to capture

	SCOPE_CYCLE_COUNTER( STAT_StreamFile );

	FMemoryWriter Ar( OutData, false, true );

	WriteCondensedMessages( Ar, TargetFrame );

	// Get cycles for all threads, so we can use that data to generate the mini-view.
	WaitTask();

	const FStatsThreadState& Stats = FStatsThreadState::GetLocalState();
	for (const auto& It : Stats.Threads)
	{
		const int64 Cycles = Stats.GetFastThreadFrameTime( TargetFrame, It.Key );
		ThreadCycles.Add( It.Key, Cycles );
	}

	SendTask();
}

void FStatsWriteFile::FinalizeSavingData( int64 FrameFileOffset )
{
	// Called from the async write thread.
	FramesInfo.Add( FStatsFrameInfo( FrameFileOffset, ThreadCycles ) );
}

/*-----------------------------------------------------------------------------
	FRawStatsWriteFile
-----------------------------------------------------------------------------*/

void FRawStatsWriteFile::SetDataDelegate( bool bSet )
{
	FStatsThreadState const& Stats = FStatsThreadState::GetLocalState();
	if (bSet)
	{
		DataDelegateHandle = Stats.NewRawStatPacket.AddRaw( this, &FRawStatsWriteFile::WriteRawStatPacket );
		if (!bWrittenOffsetToData)
		{
			const int64 FrameFileOffset = File->Tell();
			FramesInfo.Add( FStatsFrameInfo( FrameFileOffset ) );
			bWrittenOffsetToData = true;
		}
	}
	else
	{
		Stats.NewRawStatPacket.Remove( DataDelegateHandle );
	}
}

void FRawStatsWriteFile::WriteRawStatPacket( const FStatPacket* StatPacket )
{
	FMemoryWriter Ar( OutData, false, true );

	// Write stat packet.
	WriteStatPacket( Ar, (FStatPacket&)*StatPacket );
	SendTask();
}

void FRawStatsWriteFile::WriteStatPacket( FArchive& Ar, FStatPacket& StatPacket )
{
	Ar << StatPacket.Frame;
	Ar << StatPacket.ThreadId;
	int32 MyThreadType = (int32)StatPacket.ThreadType;
	Ar << MyThreadType;

	Ar << StatPacket.bBrokenCallstacks;
	// We must handle stat messages in a different way.
	int32 NumMessages = StatPacket.StatMessages.Num();
	Ar << NumMessages;
	for (const FStatMessage& Message : StatPacket.StatMessages)
	{
		WriteMessage( Ar, Message );
	}
}

/*-----------------------------------------------------------------------------
	FAsyncRawStatsReadFile
-----------------------------------------------------------------------------*/

FAsyncStatsFile::FAsyncStatsFile( FStatsReadFile* InOwner )
	: Owner( InOwner )
{}

void FAsyncStatsFile::DoWork()
{
	Owner->ReadAndProcessSynchronously();
}

void FAsyncStatsFile::Abandon()
{
	Owner->RequestStop();
}

/*-----------------------------------------------------------------------------
	FStatsReadFile
-----------------------------------------------------------------------------*/

const double FStatsReadFile::NumSecondsBetweenUpdates = 2.0;

void FStatsReadFile::ReadAndProcessSynchronously()
{
	// Read.
	ReadStats();

	// Process.
	PreProcessStats();
	ProcessStats();
	PostProcessStats();

	if (IsProcessingStopped())
	{
		SetProcessingStage( EStatsProcessingStage::SPS_Invalid );
	}
	else
	{
		SetProcessingStage( EStatsProcessingStage::SPS_Finished );
	}
}

void FStatsReadFile::ReadAndProcessAsynchronously()
{
	AsyncWork = new FAsyncTask<FAsyncStatsFile>( this );
	AsyncWork->StartBackgroundTask();
}

void FStatsReadFile::SetHistoryFrames( int32 InHistoryFrames )
{
	State.HistoryFrames = InHistoryFrames;
}

FStatsReadFile::FStatsReadFile( const TCHAR* InFilename, bool bInRawStatsFile )
	: Header( Stream.Header )
	, Reader( nullptr )
	, AsyncWork( nullptr )
	, LastUpdateTime( 0.0 )
	, Filename( InFilename )
	, NumFrames( 0 )
	, bRawStatsFile( bInRawStatsFile )
{

}

bool FStatsReadFile::PrepareLoading()
{
	const double StartTime = FPlatformTime::Seconds();

	SetProcessingStage( EStatsProcessingStage::SPS_Started );

	{
		bool bResult = true;

		ON_SCOPE_EXIT
		{
			if (!bResult)
			{
				SetProcessingStage( EStatsProcessingStage::SPS_Invalid );
			}
		};

		const int64 Size = IFileManager::Get().FileSize( *Filename );
		if (Size < 4)
		{
			UE_LOGF( LogStats, Error, "Could not open: %ls", *Filename );
			bResult = false;
			return false;
		}

		Reader = IFileManager::Get().CreateFileReader( *Filename );
		if (!Reader)
		{
			UE_LOGF( LogStats, Error, "Could not open: %ls", *Filename );
			bResult = false;
			return false;
		}

		if (!Stream.ReadHeader( *Reader ))
		{
			UE_LOGF( LogStats, Error, "Could not read, header is invalid: %ls", *Filename );
			bResult = false;
			return false;
		}

		// Verify requested stats type.
		const bool bValidStatType = Stream.Header.bRawStatsFile == bRawStatsFile;
		if (!bValidStatType)
		{
			UE_LOGF( LogStats, Error, "Could not read, invalid stats type: %ls", *Filename );
			bResult = false;
			return false;
		}

		const bool bIsFinalized = Stream.Header.IsFinalized();
		if (!bIsFinalized)
		{
			UE_LOGF( LogStats, Error, "Could not read, file is not finalized: %ls", *Filename );
			bResult = false;
			return false;
		}

		if (Stream.Header.Version < EStatMagicWithHeader::VERSION_6)
		{
			UE_LOGF( LogStats, Error, "Could not read, invalid version: %ls, expected %u, was %u", *Filename, (uint32)EStatMagicWithHeader::VERSION_6, Stream.Header.Version );
			bResult = false;
			return false;
		}

		const bool bHasCompressedData = Stream.Header.HasCompressedData();
		if (!bHasCompressedData)
		{
			UE_LOGF( LogStats, Error, "Could not read, required compressed data: %ls", *Filename );
			bResult = false;
			return false;
		}
	}

	// Read metadata.
	TArray64<FStatMessage> MetadataMessages;
	Stream.ReadFNamesAndMetadataMessages( *Reader, MetadataMessages );
	State.ProcessMetaDataOnly( MetadataMessages );

	// Find all UObject metadata messages.
	for (const FStatMessage& Meta : MetadataMessages)
	{
		const FName EncName = Meta.NameAndInfo.GetEncodedName();
		const FName RawName = Meta.NameAndInfo.GetRawName();
		const FString Desc = FStatNameAndInfo::GetShortNameFrom( RawName ).GetPlainNameString();
		const bool bContainsUObject = Desc.Contains( TEXT( "/" ) );
		if (bContainsUObject)
		{
			UObjectRawNames.Add( RawName );
		}
	}

	// Read frames offsets.
	Stream.ReadFramesOffsets( *Reader );

	// Move file pointer to the first frame or first stat packet.
	if (Stream.FramesInfo.Num() > 0)
	{
		const int64 FrameOffset0 = Stream.FramesInfo[0].FrameFileOffset;
		Reader->Seek( FrameOffset0 );
	}
	else
	{
		Reader->Seek(Reader->TotalSize());
	}

	const double TotalTime = FPlatformTime::Seconds() - StartTime;
	UE_LOGF( LogStats, Log, "Prepare loading took %.2f sec(s)", TotalTime );

	if (!bRawStatsFile)
	{
		NumFrames = Stream.FramesInfo.Num();
	}

	return true;
}

FStatsReadFile::~FStatsReadFile()
{
	RequestStop();

	FPlatformProcess::ConditionalSleep( [&]()
	{
		return !IsBusy();
	}, 1.0f );

	if (AsyncWork)
	{
		check( AsyncWork->IsDone() );

		delete AsyncWork;
		AsyncWork = nullptr;
	}

	delete Reader;
	Reader = nullptr;
}

void FStatsReadFile::ReadStats()
{
	const double StartTime = FPlatformTime::Seconds();

	if (bRawStatsFile)
	{
		ReadRawStats();
	}
	else
	{
		ReadRegularStats();
	}

	StageProgress.Set( 100 );

	if (!IsProcessingStopped())
	{
		const double TotalTime = FPlatformTime::Seconds() - StartTime;
		UE_LOGF( LogStats, Log, "Reading took %.2f sec(s)", TotalTime );

		if (bRawStatsFile)
		{
			UpdateCombinedHistoryStats();
		}
	}
	else
	{
		UE_LOGF( LogStats, Warning, "Reading stopped, abandoning" );

		if (bRawStatsFile)
		{
			// Clear all data.
			CombinedHistory.Empty();
		}
	}
}

void FStatsReadFile::ReadRawStats()
{
	// Buffer used to store the compressed and decompressed data.
	TArray<uint8> SrcArray;
	TArray<uint8> DestArray;

	// Read all packets sequentially, forced by the memory profiler which is now a part of the raw stats.
	// !!CAUTION!! Frame number in the raw stats is pointless, because it is time/cycles based, not frame based.
	// Background threads usually execute time consuming operations, so the frame number won't be valid.
	// Needs to be combined by the thread and the time, not by the frame number.

	// Update stage progress once per NumSecondsBetweenUpdates(2) seconds to avoid spamming.
	SetProcessingStage( EStatsProcessingStage::SPS_ReadStats );

	while (Reader->Tell() < Reader->TotalSize())
	{
		// Read the compressed data.
		FCompressedStatsData UncompressedData( SrcArray, DestArray );
		*Reader << UncompressedData;
		if (UncompressedData.HasReachedEndOfCompressedData())
		{
			StageProgress.Set( 100 );
			break;
		}

		FMemoryReader MemoryReader( DestArray, true );

		FStatPacket* StatPacket = new FStatPacket();
		Stream.ReadStatPacket( MemoryReader, *StatPacket );

		const int32 StatPacketFrameNum = int32( StatPacket->Frame );
		FStatPacketArray& Frame = CombinedHistory.FindOrAdd( StatPacketFrameNum );

		// Check if we need to combine packets from the same thread.
		FStatPacket** CombinedPacket = Frame.Packets.FindByPredicate( [&]( FStatPacket* Item ) -> bool
		{
			return Item->ThreadId == StatPacket->ThreadId;
		} );

		if (CombinedPacket)
		{
			(*CombinedPacket)->StatMessages += StatPacket->StatMessages;
			delete StatPacket;
		}
		else
		{
			Frame.Packets.Add( StatPacket );
			FileInfo.MaximumPacketSize = FMath::Max<int32>( FileInfo.MaximumPacketSize, (int32)StatPacket->StatMessages.GetAllocatedSize() );
		}

		UpdateReadStageProgress();
		if (IsProcessingStopped())
		{
			break;
		}

		FileInfo.TotalPacketsNum++;
	}

	// Generate frames array.
	CombinedHistory.GenerateKeyArray( Frames );
	Frames.Sort();
	// Verify that frames are sequential.
	check( Frames[Frames.Num() - 1] == Frames.Num() );

	NumFrames = Frames.Num();
}

void FStatsReadFile::ReadRegularStats()
{
	TArray<FStatMessage> PendingMessages;

	// Buffer used to store the compressed and decompressed data.
	TArray<uint8> SrcArray;
	TArray<uint8> DestArray;

	const bool bHasCompressedData = Stream.Header.HasCompressedData();
	const bool bIsFinalized = Stream.Header.IsFinalized();

	// Sanity checks.
	check( bHasCompressedData );
	check( bIsFinalized );

	// Update stage progress once per NumSecondsBetweenUpdates(2) seconds to avoid spamming.
	SetProcessingStage( EStatsProcessingStage::SPS_ReadStats );

	// Remove later.
	//Reader;
	//Header;
	//Stream;
	//State;

	for (const FStatsFrameInfo& It : Stream.FramesInfo)
	{
		PendingMessages.Reset();
		Reader->Seek( It.FrameFileOffset );

		// Read the compressed data.
		FCompressedStatsData UncompressedData( SrcArray, DestArray );
		*Reader << UncompressedData;

		// Read all messages from the uncompressed buffer.
		FMemoryReader MemoryReader( DestArray, true );
		while (MemoryReader.Tell() < MemoryReader.TotalSize())
		{
			// Read the message.
			FStatMessage Message( Stream.ReadMessage( MemoryReader, bIsFinalized ) );
			PendingMessages.Add( MoveTemp( Message ) );
		}

		// Adds a new from from the pending messages, the pending messages will be removed after the call.
		State.AddFrameFromCondensedMessages( PendingMessages );

		// Call the derived implementation.
		const TArray<FStatMessage>& CondensedMessages = *State.CondensedStackHistory.FindChecked( State.CurrentGameFrame );
		ReadStatsFrame( CondensedMessages, State.CurrentGameFrame );

		// TEST
		/*{
			// Get the stat stack root and the non frame stats
			FRawStatStackNode Stack;
			TArray<FStatMessage> NonFrameStats;
			State.UncondenseStackStats( CondensedMessages, Stack, nullptr, &NonFrameStats );
			}*/

		UpdateReadStageProgress();
		if (IsProcessingStopped())
		{
			break;
		}
	}
}

void FStatsReadFile::PreProcessStats()
{
	if (!IsProcessingStopped())
	{
		SetProcessingStage( EStatsProcessingStage::SPS_PreProcessStats );
	}
}

void FStatsReadFile::ProcessStats()
{
	if (bRawStatsFile)
	{
		if (!IsProcessingStopped())
		{
			SetProcessingStage( EStatsProcessingStage::SPS_ProcessStats );
			const double StartTime = FPlatformTime::Seconds();

			int32 CurrentStatMessageIndex = 0;

			// Raw stats callstack for this file.
			TMap<FName, FStackState> StackStates;

			// Read all stats messages for all frames, decode callstacks.
			const int32 FirstFrame = 0;
			const int32 OnePercent = FMath::Max( int32( FileInfo.TotalStatMessagesNum / 200 ), 65536 );
			int32 MessageIndexForStageProgressUpdate = 0;

			for (int32 FrameIndex = 0; FrameIndex < Frames.Num(); ++FrameIndex)
			{
				const int32 TargetFrame = Frames[FrameIndex];
				const int32 Diff = TargetFrame - FirstFrame;
				const FStatPacketArray& Frame = CombinedHistory.FindChecked( TargetFrame );

				for (int32 PacketIndex = 0; PacketIndex < Frame.Packets.Num(); PacketIndex++)
				{
					const FStatPacket& StatPacket = *Frame.Packets[PacketIndex];
					const FName& ThreadFName = State.Threads.FindChecked( StatPacket.ThreadId );

					FStackState* StackState = StackStates.Find( ThreadFName );
					if (!StackState)
					{
						StackState = &StackStates.Add( ThreadFName );
						StackState->Stack.Add( ThreadFName );
						StackState->Current = ThreadFName;
					}

					const FStatMessagesArray& Data = StatPacket.StatMessages;
					const int32 NumStatMessages = Data.Num();
					for (int32 Index = 0; Index < NumStatMessages; Index++)
					{
						CurrentStatMessageIndex++;

						const FStatMessage& Message = Data[Index];
						const EStatOperation::Type Op = Message.NameAndInfo.GetField<EStatOperation>();
						const FName RawName = Message.NameAndInfo.GetRawName();

						if (Op == EStatOperation::CycleScopeStart)
						{
							StackState->Stack.Add( RawName );
							StackState->Current = RawName;
							ProcessCycleScopeStartOperation( Message, *StackState );
						}
						// Set, Clear, Add, Subtract
						else if (Op == EStatOperation::CycleScopeEnd)
						{
							if (StackState->Stack.Num() > 1)
							{
								const FName ScopeStart = StackState->Stack.Pop();
								const FName ScopeEnd = Message.NameAndInfo.GetRawName();

								check( ScopeStart == ScopeEnd );

								StackState->Current = StackState->Stack.Last();

								// The stack should be ok, but it may be partially broken.
								// This will happen if memory profiling starts in the middle of executing a background thread.
								StackState->bIsBrokenCallstack = false;

								ProcessCycleScopeEndOperation( Message, *StackState );
							}
							else
							{
								const FName ShortName = Message.NameAndInfo.GetShortName();

								UE_LOGF( LogStats, Warning, "Broken cycle scope end %ls/%ls, current %ls",
										*ThreadFName.ToString(),
										*ShortName.ToString(),
										*StackState->Current.ToString() );

								// The stack is completely broken, only has the thread name and the last cycle scope.
								// Rollback to the thread node.
								StackState->bIsBrokenCallstack = true;
								StackState->Stack.Empty();
								StackState->Stack.Add( ThreadFName );
								StackState->Current = ThreadFName;

								//?ProcessCycleScopeEndOperation( Message, *StackState );
							}
						}
						else if (Op == EStatOperation::SpecialMessageMarker)
						{
							ProcessSpecialMessageMarkerOperation( Message, *StackState );
						}

						if (CurrentStatMessageIndex > MessageIndexForStageProgressUpdate)
						{
							UpdateProcessStageProgress( CurrentStatMessageIndex, FrameIndex, PacketIndex );
							MessageIndexForStageProgressUpdate += OnePercent;
							if (IsProcessingStopped())
							{
								Index = NumStatMessages + 1;
								PacketIndex = Frame.Packets.Num() + 1;
								FrameIndex = Frames.Num() + 1;
							}
						}
					}
				}
			}

			if (!IsProcessingStopped())
			{
				StageProgress.Set( 100 );

				const double TotalTime = FPlatformTime::Seconds() - StartTime;
				UE_LOGF( LogStats, Log, "Processing took %.2f sec(s)", TotalTime );
			}
			else
			{
				UE_LOGF( LogStats, Warning, "Processing stopped, abandoning" );
			}

			// Clear all data. We shouldn't need raw stats data at this moment.
			CombinedHistory.Empty();
		}
	}
}

void FStatsReadFile::PostProcessStats()
{
	if (!IsProcessingStopped())
	{
		SetProcessingStage( EStatsProcessingStage::SPS_PostProcessStats );
	}
}

void FStatsReadFile::UpdateReadStageProgress()
{
	const double CurrentSeconds = FPlatformTime::Seconds();
	if (CurrentSeconds > LastUpdateTime + NumSecondsBetweenUpdates)
	{
		const int32 PercentagePos = int32( 100.0*double(Reader->Tell()) / double(Reader->TotalSize()) );
		StageProgress.Set( PercentagePos );
		UE_LOGF( LogStats, Verbose, "UpdateReadStageProgress: %3i%%", PercentagePos );
		LastUpdateTime = CurrentSeconds;
	}

	// Abandon support.
	if (bShouldStopProcessing == true)
	{
		SetProcessingStage( EStatsProcessingStage::SPS_Stopped );
	}
}

void FStatsReadFile::UpdateCombinedHistoryStats()
{
	// Dump frame stats
	for (const auto& It : CombinedHistory)
	{
		const int32 FrameNum = It.Key;
		int32 FramePacketsSize = 0;
		int32 FrameStatMessages = 0;
		int32 FramePackets = It.Value.Packets.Num(); // Threads
		for (const auto& It2 : It.Value.Packets)
		{
			FramePacketsSize += (int32)It2->StatMessages.GetAllocatedSize();
			FrameStatMessages += It2->StatMessages.Num();
		}

		UE_LOGF( LogStats, Verbose, "Frame: %4i/%2i Size: %5.1f MB / %10i",
				FrameNum,
				FramePackets,
				(float)FramePacketsSize / 1024.0f / 1024.0f,
				FrameStatMessages );

		FileInfo.TotalStatMessagesNum += FrameStatMessages;
		FileInfo.TotalPacketsSize += FramePacketsSize;
	}

	UE_LOGF( LogStats, Warning, "Total PacketSize: %6.1f MB, Max: %2f MB, PacketsNum: %i, StatMessagesNum: %i, Frames: %i",
			(float)FileInfo.TotalPacketsSize / 1024.0f / 1024.0f,
			(float)FileInfo.MaximumPacketSize / 1024.0f / 1024.0f,
			FileInfo.TotalPacketsNum,
			FileInfo.TotalStatMessagesNum,
			CombinedHistory.Num() );
}

void FStatsReadFile::UpdateProcessStageProgress( const int32 CurrentStatMessageIndex, const int32 FrameIndex, const int32 PacketIndex )
{
	const double CurrentSeconds = FPlatformTime::Seconds();
	if (CurrentSeconds > LastUpdateTime + NumSecondsBetweenUpdates)
	{
		const int32 PercentagePos = int32( 100.0*CurrentStatMessageIndex / FileInfo.TotalStatMessagesNum );
		StageProgress.Set( PercentagePos );
		UE_LOGF( LogStats, Verbose, "Processing %3i%% (%10i/%10i) stat messages [Frame: %3i, Packet: %2i]", PercentagePos, CurrentStatMessageIndex, FileInfo.TotalStatMessagesNum, FrameIndex, PacketIndex );
		LastUpdateTime = CurrentSeconds;
	}

	// Abandon support.
	if (bShouldStopProcessing == true)
	{
		SetProcessingStage( EStatsProcessingStage::SPS_Stopped );
	}
}

/*-----------------------------------------------------------------------------
	Commands functionality
-----------------------------------------------------------------------------*/;

FCommandStatsFile& FCommandStatsFile::Get()
{
	static FCommandStatsFile CommandStatsFile;
	return CommandStatsFile;
}


void FCommandStatsFile::Start( const FString& Filename )
{
	Stop();
	CurrentStatsFile = new FStatsWriteFile();
	CurrentStatsFile->Start( Filename );

	StatFileActiveCounter.Increment();
}

void FCommandStatsFile::StartRaw( const FString& Filename )
{
	Stop();
	CurrentStatsFile = new FRawStatsWriteFile();
	CurrentStatsFile->Start( Filename );

	StatFileActiveCounter.Increment();
}

void FCommandStatsFile::Stop()
{
	if (CurrentStatsFile)
	{
		StatFileActiveCounter.Decrement();

		CurrentStatsFile->Stop();
		delete CurrentStatsFile;
		CurrentStatsFile = nullptr;
	}
}

void FCommandStatsFile::TestLastSaved()
{
	const FString& FilePath = FCommandStatsFile::Get().LastFileSaved;

	class FStatsTestReader : public FStatsReadFile
	{
		friend struct FStatsReader<FStatsTestReader>;
		typedef FStatsReadFile Super;

	public:
		const FStatsLoadedState& GetState() const
		{
			return State;
		}

	protected:
		/** Initialization constructor. */
		FStatsTestReader( const TCHAR* InFilename )
			: FStatsReadFile( InFilename, false )
		{
			// Keep the whole history.
			SetHistoryFrames( MAX_int32 );
		}
	};

	TUniquePtr<FStatsTestReader> Instance( FStatsReader<FStatsTestReader>::Create( *FilePath ) );

	extern void DumpHistoryFrame( FStatsThreadState const& StatsData, int64 TargetFrame, float InDumpCull = 0.0f, int32 MaxDepth = MAX_int32, TCHAR const* Filter = NULL );

	if (Instance)
	{
		Instance->ReadAndProcessSynchronously();
		const FStatsLoadedState& Loaded = Instance->GetState();

		if (Loaded.GetLatestValidFrame() < 0)
		{
			UE_LOGF( LogStats, Log, "Failed to stats file: %ls", *FilePath );
			return;
		}
		UE_LOGF( LogStats, Log, "Loaded stats file: %ls, %lld frame", *FilePath, 1 + Loaded.GetLatestValidFrame() - Loaded.GetOldestValidFrame() );
		{
			int64 TestFrame = Loaded.GetOldestValidFrame();
			UE_LOGF( LogStats, Log, "**************************** Test Frame %lld", TestFrame );
			DumpHistoryFrame( Loaded, TestFrame );
		}
	{
		int64 TestFrame = (Loaded.GetLatestValidFrame() + Loaded.GetOldestValidFrame()) / 2;
		if (Loaded.IsFrameValid( TestFrame ))
		{
			UE_LOGF( LogStats, Log, "**************************** Test Frame %lld", TestFrame );
			DumpHistoryFrame( Loaded, TestFrame );
		}
	}
	{
		int64 TestFrame = Loaded.GetLatestValidFrame();
		UE_LOGF( LogStats, Log, "**************************** Test Frame %lld", TestFrame );
		DumpHistoryFrame( Loaded, TestFrame );
	}
	}
}

#undef LOCTEXT_NAMESPACE

#endif // STATS && UE_ENABLE_STATS_FILE_DEPRECATED_IN_5_8
