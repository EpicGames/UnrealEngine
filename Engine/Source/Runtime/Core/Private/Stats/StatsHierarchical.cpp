// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stats/StatsHierarchical.h"
#include "Containers/Map.h"
#include "Algo/Reverse.h"

#define STATS_HIERARCHICAL_TIMER_FUNC() FPlatformTime::Cycles()

FStatsTreeElement::FStatsTreeElement()
	: Name(NAME_None)
	, Path(FString())
	, Invocations(0)
	, Cycles(0)
	, CyclesOfChildren(0)
	, RatioAgainstTotalInclusive(0.0)
	, RatioAgainstTotalExclusive(0.0)
	, RatioAgainstMaximumInclusive(0.0)
	, RatioAgainstMaximumExclusive(0.0)
{
}

FName FStatsTreeElement::GetFName() const
{
	return Name;
}

FString FStatsTreeElement::GetName() const
{
	return Name.ToString();
}

FString FStatsTreeElement::GetPath() const
{
	return Path;
}

uint32 FStatsTreeElement::Num(bool bInclusive) const
{
	uint32 Result = Invocations;
	if (bInclusive)
	{
		for (int32 ChildIndex = 0; ChildIndex < Children.Num(); ChildIndex++)
		{
			if (!Children[ChildIndex])
			{
				continue;
			}
			Result += Children[ChildIndex]->Num(bInclusive);
		}
	}
	return Result;
}

uint32 FStatsTreeElement::TotalCycles(bool bInclusive) const
{
	if (bInclusive)
	{
		return Cycles;
	}
	if (CyclesOfChildren > Cycles)
	{
		return 0;
	}
	return Cycles - CyclesOfChildren;
}

uint32 FStatsTreeElement::MaxCycles(bool bInclusive) const
{
	uint32 Result = TotalCycles(bInclusive);

	for (int32 ChildIndex = 0; ChildIndex < Children.Num(); ChildIndex++)
	{
		if (!Children[ChildIndex])
		{
			continue;
		}
		Result = FMath::Max<uint32>(Result, Children[ChildIndex]->MaxCycles(bInclusive));
	}

	return Result;
}

double FStatsTreeElement::TotalSeconds(bool bInclusive) const
{
	return TotalCycles(bInclusive) * FPlatformTime::GetSecondsPerCycle();
}

double FStatsTreeElement::AverageSeconds(bool bInclusive) const
{
	ensure(Invocations > 0);
	return TotalSeconds(bInclusive) / double(Invocations);
}

double FStatsTreeElement::Contribution(bool bAgainstMaximum, bool bInclusive) const
{
	if (bInclusive)
	{
		return bAgainstMaximum ? RatioAgainstMaximumInclusive : RatioAgainstTotalInclusive;
	}
	return bAgainstMaximum ? RatioAgainstMaximumExclusive : RatioAgainstTotalExclusive;
}

const TArray<TSharedPtr<FStatsTreeElement>>& FStatsTreeElement::GetChildren() const
{
	return Children;
}

FStatsTreeElement* FStatsTreeElement::FindChild(const FString& InPath)
{
	if (InPath.IsEmpty())
	{
		return this;
	}

	FString Left, Right;
	if (InPath.Split(TEXT("."), &Left, &Right))
	{
		FStatsTreeElement* Child = FindChild(Left);
		if (Child == nullptr)
		{
			return nullptr;
		}
		return Child->FindChild(Right);
	}

	for (int32 ChildIndex = 0; ChildIndex < Children.Num(); ChildIndex++)
	{
		if (!Children[ChildIndex])
		{
			continue;
		}
		if (Children[ChildIndex]->GetName().Equals(InPath))
		{
			return Children[ChildIndex].Get();
		}
	}

	return nullptr;
}

void FStatsTreeElement::UpdatePostMeasurement(double InCyclesPerTimerToRemove)
{
	struct Local
	{
		static void RemoveCyclesForTimers(FStatsTreeElement* Element, double CyclesPerTimerToRemove)
		{
			if (!Element)
			{
				return;
			}

			if (Element->Cycles > 0)
			{
				uint32 InvocationsOfChildren = Element->Num(true) - Element->Num(false);
				uint32 TimingCostOfChildren = (uint32)FMath::RoundToDouble(double(InvocationsOfChildren) * CyclesPerTimerToRemove + 0.499);
				if (Element->Cycles < TimingCostOfChildren)
				{
					Element->Cycles = 0;
				}
				else
				{
					Element->Cycles -= TimingCostOfChildren;
				}
			}

			for (TSharedPtr<FStatsTreeElement>& Child : Element->Children)
			{
				FStatsTreeElement* ChildPtr = Child.Get();
				if (ChildPtr)
				{
					RemoveCyclesForTimers(ChildPtr, CyclesPerTimerToRemove);
				}
			}
		}

		static void ComputeChildrenCycles(FStatsTreeElement* Element)
		{
			if (!Element)
			{
				return;
			}

			if (Element->Children.Num() > 0)
			{
				Element->CyclesOfChildren = 0;
				for (TSharedPtr<FStatsTreeElement>& Child : Element->Children)
				{
					FStatsTreeElement* ChildPtr = Child.Get();
					if (ChildPtr)
					{
						Element->CyclesOfChildren += ChildPtr->Cycles;
						ComputeChildrenCycles(ChildPtr);
					}
				}
			}
		}

		static void UpdateRatios(FStatsTreeElement* Element, uint32 ParentCyclesInclusive, uint32 ParentCyclesExclusive, uint32 MaxCyclesInclusive, uint32 MaxCyclesExclusive)
		{
			if (!Element)
			{
				return;
			}

			Element->RatioAgainstTotalInclusive = 0.0;
			Element->RatioAgainstTotalExclusive = 0.0;
			Element->RatioAgainstMaximumInclusive = 0.0;
			Element->RatioAgainstMaximumExclusive = 0.0;

			if (ParentCyclesInclusive > 0)
			{
				Element->RatioAgainstTotalInclusive = double(Element->TotalCycles(true)) / double(ParentCyclesInclusive);
			}
			if (ParentCyclesExclusive > 0)
			{
				Element->RatioAgainstTotalExclusive = double(Element->TotalCycles(false)) / double(ParentCyclesExclusive);
			}
			if (MaxCyclesInclusive > 0)
			{
				Element->RatioAgainstMaximumInclusive = double(Element->TotalCycles(true)) / double(MaxCyclesInclusive);;
			}
			if (MaxCyclesExclusive > 0)
			{
				Element->RatioAgainstMaximumExclusive = double(Element->TotalCycles(true)) / double(MaxCyclesExclusive);
			}

			ParentCyclesInclusive = ParentCyclesExclusive = 0;

			for (TSharedPtr<FStatsTreeElement>& Child : Element->Children)
			{
				FStatsTreeElement* ChildPtr = Child.Get();
				if (ChildPtr)
				{
					ParentCyclesInclusive += ChildPtr->TotalCycles(true);
					ParentCyclesExclusive += ChildPtr->TotalCycles(false);
				}
			}

			for (TSharedPtr<FStatsTreeElement>& Child : Element->Children)
			{
				FStatsTreeElement* ChildPtr = Child.Get();
				if (ChildPtr)
				{
					UpdateRatios(ChildPtr, ParentCyclesInclusive, ParentCyclesExclusive, MaxCyclesInclusive, MaxCyclesExclusive);
				}
			}
		}
	};

	// remove the cycles for the timers
	if (InCyclesPerTimerToRemove > UE_SMALL_NUMBER)
	{
		Local::RemoveCyclesForTimers(this, InCyclesPerTimerToRemove);
	}

	// compute the exclusive cycles
	Local::ComputeChildrenCycles(this);

	// find the max inclusive and exclusive cycles
	uint32 MaxCyclesInclusive = MaxCycles(true);
	uint32 MaxCyclesExclusive = MaxCycles(false);

	// update the ratios used for the UI
	Local::UpdateRatios(this, 0, 0, MaxCyclesInclusive, MaxCyclesExclusive);
}

bool FStatsHierarchical::bEnabled = false;

namespace
{
FStatsTreeElement& GetStatsHierarchicalLastMeasurement()
{
	static FStatsTreeElement LastMeasurement;
	return LastMeasurement;
}
}

TArray<FStatsHierarchical::FHierarchicalStatEntry> FStatsHierarchical::Entries;

void FStatsHierarchical::BeginMeasurements()
{
#if STATS

	bEnabled = true;
	Entries.Reset();

#endif
}

bool FStatsHierarchical::IsEnabled()
{
#if STATS

	return bEnabled;

#else 

	return false;

#endif
}

FStatsTreeElement FStatsHierarchical::EndMeasurements(FStatsTreeElement MeasurementsToMerge, bool bAddUntrackedElements)
{
#if STATS

	bEnabled = false;

	struct FTreeElementInfo
	{
		FStatsTreeElement Element;
		uint32 BeginCycles;
	};

	TMap<FString, FTreeElementInfo> Elements;

	struct Local
	{
		static void InsertAllElementsToMap(TMap<FString, FTreeElementInfo>& InElements, const FStatsTreeElement* Element)
		{
			if (!Element)
			{
				return;
			}
			if (Element->GetFName() == GetUntrackedTimeName())
			{
				return;
			}

			if (Element->GetFName() != NAME_None)
			{
				FTreeElementInfo Info;
				Info.Element.Name = Element->GetFName();
				Info.Element.Path = Element->GetPath();
				Info.Element.Invocations = Element->Num();
				Info.Element.Cycles = Element->TotalCycles();
				Info.BeginCycles = 0;
				InElements.Add(Info.Element.GetPath(), Info);
			}

			for (const TSharedPtr<FStatsTreeElement>& Child : Element->Children)
			{
				InsertAllElementsToMap(InElements, Child.Get());
			}
		}
	};

	if (MeasurementsToMerge.Name == NAME_None && MeasurementsToMerge.Children.Num() > 0)
	{
		Local::InsertAllElementsToMap(Elements, &MeasurementsToMerge);
	}

	TArray<FString> Paths;
	for (int32 EntryIndex = 0; EntryIndex < Entries.Num(); EntryIndex++)
	{
		const FHierarchicalStatEntry& Entry = Entries[EntryIndex];
		if (Entry.Label != nullptr)
		{
			FString Label = Entry.Label;
			Label.ReplaceCharInline(L'.', L'_');
			FString Path;
			if (Paths.Num() > 0)
			{
				Path = FString::Printf(TEXT("%s.%s"), *Paths.Last(), *Label);
			}
			else
			{
				Path = Label;
			}

			FTreeElementInfo* ExistingInfo = Elements.Find(Path);
			if (ExistingInfo == nullptr)
			{
				FTreeElementInfo Info;
				Info.Element = FStatsTreeElement();
				Info.Element.Name = FName(*Label);
				Info.Element.Path = Path;
				Info.BeginCycles = Entry.Cycles;
				Elements.Add(Path, Info);
			}
			else
			{
				ExistingInfo->BeginCycles = Entry.Cycles;
			}

			Paths.Add(Path);
		}
		else
		{
			ensure(Paths.Num() > 0);
			FString LastPath = Paths.Pop();

			FTreeElementInfo& Info = Elements.FindChecked(LastPath);
			uint32 Cycles = Entry.Cycles - Info.BeginCycles;
			Info.Element.Cycles += Cycles;
			Info.Element.Invocations++;
		}
	}

	// if we are hitting this it means we have an unbalanced
	// scope. somebody is calling EndMeasurements before the last scope is destroyed.
	ensure(Paths.Num() == 0);

	FStatsTreeElement CurrentMeasurement;

	for (const TPair<FString, FTreeElementInfo>& Pair : Elements)
	{
		FString Left, Right;
		Pair.Key.Split(TEXT("."), &Left, &Right, ESearchCase::IgnoreCase, ESearchDir::FromEnd);
		FStatsTreeElement* Parent = CurrentMeasurement.FindChild(Left);
		check(Parent);
		Parent->Children.Add(MakeShared<FStatsTreeElement>(Pair.Value.Element));
	}

	CurrentMeasurement.Invocations = 1;

	// measure the time for the timing function
	Entries.Reset();
	bEnabled = true;
	uint32 TimerStartCycles = STATS_HIERARCHICAL_TIMER_FUNC();
	for (int32 TimingIndex = 0; TimingIndex < 100000; TimingIndex++)
	{
		DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC();
	}
	uint32 TimerEndCycles = STATS_HIERARCHICAL_TIMER_FUNC();
	Entries.Reset();
	bEnabled = false;

	uint32 TimerDelta = TimerEndCycles - TimerStartCycles;

	double CyclesPerTimer = double(TimerDelta) / 100000.0;
	CurrentMeasurement.UpdatePostMeasurement(CyclesPerTimer);

	if (bAddUntrackedElements)
	{
		for (const TPair<FString, FTreeElementInfo>& Pair : Elements)
		{
			FStatsTreeElement* Element = CurrentMeasurement.FindChild(Pair.Key);
			check(Element);

			if (Element->Children.Num() == 0)
			{
				continue;
			}

			if (Element->CyclesOfChildren < Element->Cycles && (Element->Cycles - Element->CyclesOfChildren) > CyclesPerTimer)
			{
				FStatsTreeElement UntrackedElement;
				UntrackedElement.Name = GetUntrackedTimeName();
				UntrackedElement.Path = FString::Printf(TEXT("%s.%s"), *Element->GetPath(), *UntrackedElement.GetName());
				UntrackedElement.Invocations = 1;
				UntrackedElement.Cycles = Element->Cycles - Element->CyclesOfChildren;
				Element->Children.Add(MakeShared<FStatsTreeElement>(UntrackedElement));
			}
		}

		CurrentMeasurement.UpdatePostMeasurement();
	}

	GetStatsHierarchicalLastMeasurement() = CurrentMeasurement;
#endif
	return GetLastMeasurements();
}

FStatsTreeElement FStatsHierarchical::GetLastMeasurements()
{
#if STATS

	ensure(!bEnabled);

#endif

	return GetStatsHierarchicalLastMeasurement();
}

void FStatsHierarchical::DumpMeasurements(FMessageLog& Log, bool bSortByDuration)
{
#if STATS
	if (bEnabled)
	{
		return;
	}

	struct Local
	{
		static void DumpEntry(const FStatsTreeElement* Element, FMessageLog& InLog, bool bInSortByDuration = true, FString Prefix = TEXT("+"))
		{
			if (!Element)
			{
				return;
			}
			if (Element->GetFName() != NAME_None)
			{
				double TotalMS= Element->TotalSeconds() * 1000.0;
				double AverageMS = Element->AverageSeconds() * 1000.0;
				FText Message;
				if (Element->Num() > 1)
				{
					Message = FText::FromString(FString::Printf(TEXT("%s %s %.03fms (%d runs, %.03fms avg)"), *Prefix, *Element->GetName(), (float)TotalMS, (int32)Element->Num(), (float)AverageMS));
				}
				else
				{
					Message = FText::FromString(FString::Printf(TEXT("%s %s %.03fms (1 run)"), *Prefix, *Element->GetName(), (float)TotalMS));
				}

				InLog.Info(Message);
				Prefix = Prefix + TEXT("---");
			}

			const TArray<TSharedPtr<FStatsTreeElement>>& Children =  Element->GetChildren();
			for (const TSharedPtr<FStatsTreeElement>& Child : Children)
			{
				DumpEntry(Child.Get(), InLog, bInSortByDuration, Prefix);
			}
		}
	};

	Log.Info(FText::FromString(TEXT("----------------------------------------------")));
	Local::DumpEntry(&(GetStatsHierarchicalLastMeasurement()), Log, bSortByDuration);

#endif
}

FName FStatsHierarchical::GetUntrackedTimeName()
{
	return TEXT("__UNTRACKED__");
}

void FStatsHierarchical::BeginMeasurement(const ANSICHAR * Label)
{
#if STATS

	if (!bEnabled)
	{
		return;
	}
	if (Entries.Num() == 0)
	{
		Entries.Reserve(1024 * 1024);
	}
	else if (Entries.Num() == Entries.Max())
	{
		Entries.Reserve(Entries.Max() * 2);
	}
	Entries.Add(FHierarchicalStatEntry(Label, STATS_HIERARCHICAL_TIMER_FUNC()));

#endif
}

void FStatsHierarchical::EndMeasurement()
{
#if STATS

	if (!bEnabled)
	{
		return;
	}
	Entries.Add(FHierarchicalStatEntry(nullptr, STATS_HIERARCHICAL_TIMER_FUNC()));

#endif
}
