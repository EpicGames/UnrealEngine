// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stats/StatsMisc.h"
#include "Stats/Stats.h"
#include "Logging/LogMacros.h"
#include "CoreGlobals.h"

FConditionalScopeLogTime::FConditionalScopeLogTime( bool bCondition, const WIDECHAR* InName, FTotalTimeAndCount* InCumulative /*= nullptr */, EScopeLogTimeUnits InUnits /*= ScopeLog_Milliseconds */ )
: StartTime( bCondition ? FPlatformTime::Seconds() : 0.0 )
, Name( InName )
, Cumulative( InCumulative )
, Units( bCondition ? InUnits : ScopeLog_DontLog )
{}

FConditionalScopeLogTime::FConditionalScopeLogTime( bool bCondition, const ANSICHAR* InName, FTotalTimeAndCount* InCumulative /*= nullptr*/, EScopeLogTimeUnits InUnits /*= ScopeLog_Milliseconds */ )
: StartTime( bCondition ? FPlatformTime::Seconds() : 0.0 )
, Name( InName )
, Cumulative( InCumulative )
, Units( bCondition ? InUnits : ScopeLog_DontLog )
{}

FConditionalScopeLogTime::~FConditionalScopeLogTime()
{
	if (Units != ScopeLog_DontLog)
	{
		const double ScopedTime = FPlatformTime::Seconds() - StartTime;
		const FString DisplayUnitsString = GetDisplayUnitsString();
		if( Cumulative )
		{
			Cumulative->Key += ScopedTime;
			Cumulative->Value++;

			const double Average = Cumulative->Key / (double)Cumulative->Value;
			UE_LOGF( LogStats, Log, "%32ls - %6.3f %ls - Total %6.2f s / %5u / %6.3f %ls", *Name, GetDisplayScopedTime(ScopedTime), *DisplayUnitsString, Cumulative->Key, Cumulative->Value, GetDisplayScopedTime(Average), *DisplayUnitsString );
		}
		else
		{
			UE_LOGF( LogStats, Log, "%32ls - %6.3f %ls", *Name, GetDisplayScopedTime(ScopedTime), *DisplayUnitsString );
		}
	}
}

double FConditionalScopeLogTime::GetDisplayScopedTime(double InScopedTime) const
{
	if (Units == ScopeLog_Seconds)
	{
		return InScopedTime;
	}

	return InScopedTime * 1000.0f;
}

FString FConditionalScopeLogTime::GetDisplayUnitsString() const
{
	if (Units == ScopeLog_Seconds)
	{
		return TEXT("s");
	}

	return TEXT("ms");
}
