// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stats/StatsSystem.h"

#include "Async/TaskGraphInterfaces.h"
#include "Containers/Ticker.h"
#include "CoreGlobals.h"
#include "HAL/LowLevelMemTracker.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreDelegates.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "Stats/Stats.h"
#include "Stats/StatsData.h"
#include "Stats/StatsSystemTypes.h"
#include "Stats/StatIgnoreList.h"

// Declared in GlobalStats.inl
DEFINE_STAT(STAT_FrameTime);
DEFINE_STAT(STAT_NamedMarker);
DEFINE_STAT(STAT_SecondsPerCycle);

namespace UE::Stats
{
#if STATS
	// The engine stats system maintains its own frame counter. We need to propagate this to
	// the end-of-pipe thread so that stat data it submits is attributed to the correct frame.
	// This is done inside AdvanceRenderingThreadStats.
	CORE_API TOptional<int64> FStats::StatsFrameRT;
#endif

	TAtomic<int32> FStats::GameThreadStatsFrame(1);

	namespace Private
	{
		static void OnInit()
		{
#if UE_STATS_ALLOW_PER_THREAD_IGNORELIST
			InitializeIgnoreList();
#endif
		}
	}

	void FStats::Init()
	{
#if UE_STATS_ALLOW_PER_THREAD_IGNORELIST
		if (GConfig != nullptr && GConfig->IsReadyForUse())
		{
			InitializeIgnoreList();
		}
		else
		{
			FCoreDelegates::OnInit.AddStatic(&Private::OnInit);
		}
#endif // UE_STATS_ALLOW_PER_THREAD_IGNORELIST
	}

	void FStats::AdvanceFrame(bool bDiscardCallstack, const FOnAdvanceRenderingThreadStats& AdvanceRenderingThreadStatsDelegate /*= FOnAdvanceRenderingThreadStats()*/)
	{
#if STATS
		TRACE_CPUPROFILER_EVENT_SCOPE(FStats::AdvanceFrame);
		LLM_SCOPE(ELLMTag::Stats);
		check(IsInGameThread());
		static int32 PrimaryDisableChangeTagStartFrame = -1;
		int64 Frame = ++GameThreadStatsFrame;

		if (bDiscardCallstack)
		{
			FThreadStats::FrameDataIsIncomplete(); // we won't collect call stack stats this frame
		}
		if (PrimaryDisableChangeTagStartFrame == -1)
		{
			PrimaryDisableChangeTagStartFrame = FThreadStats::PrimaryDisableChangeTag();
		}
		if (!FThreadStats::IsCollectingData() || PrimaryDisableChangeTagStartFrame != FThreadStats::PrimaryDisableChangeTag())
		{
			Frame = -Frame; // mark this as a bad frame
		}

		// Update the seconds per cycle.
		SET_FLOAT_STAT(STAT_SecondsPerCycle, FPlatformTime::GetSecondsPerCycle());

		FThreadStats::AddMessage(FStatConstants::AdvanceFrame.GetEncodedName(), EStatOperation::AdvanceFrameEventGameThread, Frame); // we need to flush here if we aren't collecting stats to make sure the meta data is up to date

		if (AdvanceRenderingThreadStatsDelegate.IsBound())
		{
			AdvanceRenderingThreadStatsDelegate.Execute(bDiscardCallstack, Frame, PrimaryDisableChangeTagStartFrame);
		}
		else
		{
			// There is no rendering thread, so this message is sufficient to make stats happy and don't leak memory.
			FThreadStats::AddMessage(FStatConstants::AdvanceFrame.GetEncodedName(), EStatOperation::AdvanceFrameEventRenderThread, Frame);
			FThreadStats::AddMessage(FStatConstants::AdvanceFrame.GetEncodedName(), EStatOperation::AdvanceFrameEventEndOfPipe, Frame);
		}

		FThreadStats::ExplicitFlush(bDiscardCallstack);
		FThreadStats::WaitForStats();
		PrimaryDisableChangeTagStartFrame = FThreadStats::PrimaryDisableChangeTag();
#endif
	}

	void FStats::TickCommandletStats()
	{
		if (EnabledForCommandlet())
		{
			//FThreadStats* ThreadStats = FThreadStats::GetThreadStats();
			//check( ThreadStats->ScopeCount == 0 && TEXT( "FStats::TickCommandletStats must be called outside any scope counters" ) );

			FTaskGraphInterface::Get().ProcessThreadUntilIdle(ENamedThreads::GameThread);
			FTSTicker::GetCoreTicker().Tick(1 / 60.0f);

			FStats::AdvanceFrame(false);
		}
	}

	bool FStats::EnabledForCommandlet()
	{
		static bool bHasStatsForCommandletsToken = HasLoadTimeStatsForCommandletToken() || HasLoadTimeFileForCommandletToken();
		return bHasStatsForCommandletsToken;
	}

	bool FStats::HasLoadTimeStatsForCommandletToken()
	{
		static bool bHasLoadTimeStatsForCommandletToken = FParse::Param(FCommandLine::Get(), TEXT("LoadTimeStatsForCommandlet"));
		return bHasLoadTimeStatsForCommandletToken;
	}

	bool FStats::HasLoadTimeFileForCommandletToken()
	{
		static bool bHasLoadTimeFileForCommandletToken = FParse::Param(FCommandLine::Get(), TEXT("LoadTimeFileForCommandlet"));
		return bHasLoadTimeFileForCommandletToken;
	}
} // namespace UE::Stats