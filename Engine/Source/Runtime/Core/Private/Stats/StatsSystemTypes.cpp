// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stats/StatsSystemTypes.h"

#include "Async/AsyncWork.h"
#include "Async/TaskGraphInterfaces.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Ticker.h"
#include "Containers/UnrealString.h"
#include "HAL/IConsoleManager.h"
#include "HAL/LowLevelMemTracker.h"
#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "HAL/ThreadSafeCounter.h"
#include "HAL/UnrealMemory.h"
#include "Logging/LogMacros.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CString.h"
#include "Misc/Parse.h"
#include "Misc/StringBuilder.h"
#include "Misc/ScopeLock.h"
#include "Misc/CommandLine.h"
#include "Misc/SingleThreadRunnable.h"
#include "Stats/Stats.h"
#include "Stats/StatsCommand.h"
#include "Stats/StatsSystem.h"
#include "Stats/StatsData.h"
#include "String/Find.h"
#include "Tasks/Pipe.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/NameTypes.h"

/*-----------------------------------------------------------------------------
	Global
-----------------------------------------------------------------------------*/

struct FStats2Globals
{
	static void Get()
	{
#if	STATS
		FStartupMessages::Get();
		IStatGroupEnableManager::Get();
#endif // STATS
	}
};

#if STATS
CORE_API UE::Tasks::FPipe GStatsPipe{ UE_SOURCE_LOCATION };
#endif

static struct FForceInitAtBootFStats2 : public TForceInitAtBoot<FStats2Globals>
{} FForceInitAtBootFStats2;

/*-----------------------------------------------------------------------------
	DebugLeakTest, for the stats based memory profiler
-----------------------------------------------------------------------------*/

#if	!UE_BUILD_SHIPPING

static TAutoConsoleVariable<int32> CVarEnableLeakTest(
	TEXT("EnableLeakTest"),
	0,
	TEXT("If set to 1, enables leak test, for testing stats based memory profiler")
);

static void DebugLeakTest()
{
	if (CVarEnableLeakTest.GetValueOnGameThread() == 1)
	{
		if (GFrameCounter == 60)
		{
			UE::Stats::DirectStatsCommand(TEXT("stat namedmarker Frame-060"), true);
		}

		if (GFrameCounter == 120)
		{
			UE::Stats::DirectStatsCommand(TEXT("stat namedmarker Frame-120"), true);
		}


		if (GFrameCounter == 240)
		{
			UE::Stats::DirectStatsCommand(TEXT("stat namedmarker Frame-240"), true);
		}

		if (GFrameCounter == 300)
		{
			RequestEngineExit(TEXT("DebugLeakTest hit frame 300"));
		}

		// Realloc.
		static TArray<uint8> Array;
		static int32 Initial = 1;
		{
			DECLARE_SCOPE_CYCLE_COUNTER(TEXT("LeakTest::Realloc"), Stat_LeakTest_Realloc, STATGROUP_Quick);
			Array.AddZeroed(Initial);
			Initial += 100;
		}

		if (GFrameCounter == 300)
		{
			UE_LOG(LogTemp, Warning, TEXT("Stat_ReallocTest: %" SIZE_T_FMT " / %i"), Array.GetAllocatedSize(), Initial);
		}

		// General memory leak.
		{
			DECLARE_SCOPE_CYCLE_COUNTER(TEXT("LeakTest::NewInt8"), Stat_LeakTest_NewInt8, STATGROUP_Quick);
			int8* Leak = new int8[1000 * 1000];
		} //-V773


		if (GFrameCounter < 250)
		{
			// Background threads memory test.
			struct FAllocTask
			{
				static void Alloc()
				{
					DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FAllocTask::Alloc"), Stat_FAllocTask_Alloc, STATGROUP_Quick);

					int8* IntAlloc = new int8[112233];
					int8* LeakTask = new int8[100000];
					delete[] IntAlloc;
				} //-V773
			};

			for (int32 Index = 0; Index < 40; ++Index)
			{
				FSimpleDelegateGraphTask::CreateAndDispatchWhenReady(FSimpleDelegateGraphTask::FDelegate::CreateStatic(FAllocTask::Alloc), TStatId());
			}

			class FAllocPool : public FNonAbandonableTask
			{
			public:
				void DoWork()
				{
					DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FAllocPool::DoWork"), Stat_FAllocPool_DoWork, STATGROUP_Quick);

					int8* IntAlloc = new int8[223311];
					int8* LeakTask = new int8[100000];
					delete[] IntAlloc;
				} //-V773

				TStatId GetStatId() const
				{
					return TStatId();
				}
			};

			for (int32 Index = 0; Index < 40; ++Index)
			{
				(new FAutoDeleteAsyncTask<FAllocPool>())->StartBackgroundTask();
			}
		}

		for (int32 Index = 0; Index < 40; ++Index)
		{
			DECLARE_SCOPE_CYCLE_COUNTER(TEXT("DebugLeakTest::Alloc"), Stat_LeakTest_Alloc, STATGROUP_Quick);

			int8* IntAlloc = new int8[331122];
			int8* LeakTask = new int8[100000];
			delete[] IntAlloc;
		} //-V773

		if (IsEngineExitRequested())
		{
			// If we are writing stats data, stop it now.
			UE::Stats::DirectStatsCommand(TEXT("stat stopfile"), true);
		}
	}
}

#endif // !UE_BUILD_SHIPPING

/* Todo

ST_int64 combined with IsPackedCCAndDuration is fairly bogus, ST_uint32_pair should probably be a first class type. IsPackedCCAndDuration can stay, it says what kind of data is in the pair. remove FromPackedCallCountDuration_Duration et al

averages stay as int64, it might be better if they were floats, but then we would probably want a ST_float_pair too.

//@todo merge these two after we have redone the user end

set a DEBUG_STATS define that allows this all to be debugged. Otherwise, inline and get rid of checks to the MAX. Also maybe just turn off stats in debug or debug game.

It should be possible to load stats data without STATS

//@todo this probably goes away after we redo the programmer interface

We are only saving condensed frames. For the "core view", we want a single frame capture of FStatsThreadState::History. That is the whole enchillada including start and end time for every scope.

//@todo Legacy API, phase out after we have changed the programmer-facing APU

stats2.h, stats2.cpp, statsrender2.cpp - get rid of the 2s.

//@todo, delegate under a global bool?

//@todo The rest is all horrid hacks to bridge the game between the old and the new system while we finish the new system

//@todo split header

//@todo metadata probably needs a different queue, otherwise it would be possible to load a module, run code in a thread and have the events arrive before the meta data

delete "wait for render commands", and generally be on the look out for stats that are never used, also stat system stuff

sweep INC type things for dump code that calls INC in a loop instead of just calling INC_BY once

FORCEINLINE_STATS void Start(FName InStatId)
{
check(InStatId != NAME_None);

^^^should be a checkstats


*/

DECLARE_DWORD_COUNTER_STAT(TEXT("Frame Packets Received"), STAT_StatFramePacketsRecv, STATGROUP_StatSystem);

DECLARE_CYCLE_STAT(TEXT("StatsNew Tick"), STAT_StatsNewTick, STATGROUP_StatSystem);
DECLARE_CYCLE_STAT(TEXT("Parse Meta"), STAT_StatsNewParseMeta, STATGROUP_StatSystem);
DECLARE_CYCLE_STAT(TEXT("Scan For Advance"), STAT_ScanForAdvance, STATGROUP_StatSystem);
DECLARE_CYCLE_STAT(TEXT("Add To History"), STAT_StatsNewAddToHistory, STATGROUP_StatSystem);
DECLARE_CYCLE_STAT(TEXT("Flush Raw Stats"), STAT_FlushRawStats, STATGROUP_StatSystem);

DECLARE_MEMORY_STAT(TEXT("Stats Descriptions"), STAT_StatDescMemory, STATGROUP_StatSystem);

#if STATS

TStatIdData TStatId::TStatId_NAME_None;

/*-----------------------------------------------------------------------------
	FStartupMessages
-----------------------------------------------------------------------------*/

void FStartupMessages::AddThreadMetadata(const FName InThreadName, uint32 InThreadID)
{
	// Make unique name.
	const FString ThreadName = FStatsUtils::BuildUniqueThreadName(InThreadID);

	FStartupMessages::AddMetadata(InThreadName, *ThreadName, STAT_GROUP_TO_FStatGroup(STATGROUP_Threads)::GetGroupName(), STAT_GROUP_TO_FStatGroup(STATGROUP_Threads)::GetGroupCategory(), STAT_GROUP_TO_FStatGroup(STATGROUP_Threads)::GetDescription(), true, EStatDataType::ST_int64, true, false);
}


void FStartupMessages::AddMetadata(FName InStatName, const TCHAR* InStatDesc, const char* InGroupName, const char* InGroupCategory, const TCHAR* InGroupDesc, bool bShouldClearEveryFrame, EStatDataType::Type InStatType, bool bCycleStat, bool bSortByName, FPlatformMemory::EMemoryCounterRegion InMemoryRegion /*= FPlatformMemory::MCR_Invalid*/)
{
	LLM_SCOPE(ELLMTag::Stats);
	UE::TScopeLock Lock(CriticalSection);

	DelayedMessages.Emplace(InGroupName, EStatDataType::ST_None, "Groups", InGroupCategory, InGroupDesc, false, false, bSortByName);
	DelayedMessages.Emplace(InStatName, InStatType, InGroupName, InGroupCategory, InStatDesc, bShouldClearEveryFrame, bCycleStat, bSortByName, InMemoryRegion);
}


FStartupMessages& FStartupMessages::Get()
{
	static FStartupMessages* Messages = NULL;
	if (!Messages)
	{
		LLM_SCOPE(ELLMTag::Stats);
		check(IsInGameThread());
		Messages = new FStartupMessages;
	}
	return *Messages;
}

/*-----------------------------------------------------------------------------
	FThreadSafeStaticStatBase
-----------------------------------------------------------------------------*/

const TStatIdData* FThreadSafeStaticStatBase::DoSetup(FName InStatName, const TCHAR* InStatDesc, const char* InGroupName, const char* InGroupCategory, const TCHAR* InGroupDesc, bool bDefaultEnable, bool bShouldClearEveryFrame, EStatDataType::Type InStatType, bool bCycleStat, bool bSortByName, FPlatformMemory::EMemoryCounterRegion InMemoryRegion) const
{
	// send meta data, we don't use normal messages because the stats thread might not be running yet
	FStartupMessages::Get().AddMetadata(InStatName, InStatDesc, InGroupName, InGroupCategory, InGroupDesc, bShouldClearEveryFrame, InStatType, bCycleStat, bSortByName, InMemoryRegion);

	TStatIdData const* LocalHighPerformanceEnable(IStatGroupEnableManager::Get().GetHighPerformanceEnableForStat(InStatName, InGroupName, InGroupCategory, bDefaultEnable, bShouldClearEveryFrame, InStatType, InStatDesc, bCycleStat, bSortByName, InMemoryRegion).GetRawPointer());
	TStatIdData const* OldHighPerformanceEnable = HighPerformanceEnable.Exchange(LocalHighPerformanceEnable);
	check(!OldHighPerformanceEnable || LocalHighPerformanceEnable == OldHighPerformanceEnable); // we are assigned two different groups?

	return LocalHighPerformanceEnable;
}

/*-----------------------------------------------------------------------------
	FStatGroupEnableManager
-----------------------------------------------------------------------------*/

DEFINE_LOG_CATEGORY_STATIC(LogStatGroupEnableManager, Log, All);

class FStatGroupEnableManager : public IStatGroupEnableManager
{
	struct FGroupEnable
	{
		TMap<FName, TStatIdData*> NamesInThisGroup;
		TMap<FName, TStatIdData*> AlwaysEnabledNamesInThisGroup;
		bool DefaultEnable;
		bool CurrentEnable;

		FGroupEnable(bool InDefaultEnable)
			: DefaultEnable(InDefaultEnable)
			, CurrentEnable(InDefaultEnable)
		{
		}
	};

	enum
	{
		/** Number of stats pointer allocated per block. */
		NUM_PER_BLOCK = 16384,
	};


	TMap<FName, FGroupEnable> HighPerformanceEnable;

	/** Used to synchronize the access to the high performance stats groups. */
	FTransactionallySafeCriticalSection SynchronizationObject;

	/** Pointer to the long name in the names block. */
	TArray<TArray<TStatIdData>> StatIDs;

	/** Holds the amount of memory allocated for the stats descriptions. */
	FThreadSafeCounter MemoryCounter;

	// these control what happens to groups that haven't been registered yet
	TMap<FName, bool> EnableForNewGroup;
	bool EnableForNewGroups;
	bool UseEnableForNewGroups;

	void EnableStat(FName StatName, TStatIdData* DisablePtr)
	{
		DisablePtr->Name = NameToMinimalName(StatName);
	}

	void DisableStat(TStatIdData* DisablePtr)
	{
		DisablePtr->Name = TStatId::GetStatNone().Name.Load();
	}

public:
	FStatGroupEnableManager()
		: EnableForNewGroups(false)
		, UseEnableForNewGroups(false)
	{
		check(IsInGameThread());
	}

	virtual void UpdateMemoryUsage() override
	{
		// Update the stats descriptions memory usage.
		const int32 MemoryUsage = MemoryCounter.GetValue();
		SET_MEMORY_STAT(STAT_StatDescMemory, MemoryUsage);
	}

	virtual void SetHighPerformanceEnableForGroup(FName Group, bool Enable) override
	{
		UE::TScopeLock ScopeLock(SynchronizationObject);
		FThreadStats::PrimaryDisableChangeTagLockAdd();
		FGroupEnable* Found = HighPerformanceEnable.Find(Group);
		if (Found)
		{
			Found->CurrentEnable = Enable;
			if (Enable)
			{
				for (auto It = Found->NamesInThisGroup.CreateIterator(); It; ++It)
				{
					EnableStat(It.Key(), It.Value());
				}
			}
			else
			{
				for (auto It = Found->NamesInThisGroup.CreateIterator(); It; ++It)
				{
					DisableStat(It.Value());
				}
			}
		}
		FThreadStats::PrimaryDisableChangeTagLockSubtract();
	}

	virtual void SetHighPerformanceEnableForAllGroups(bool Enable) override
	{
		UE::TScopeLock ScopeLock(SynchronizationObject);
		FThreadStats::PrimaryDisableChangeTagLockAdd();
		for (auto It = HighPerformanceEnable.CreateIterator(); It; ++It)
		{
			It.Value().CurrentEnable = Enable;
			if (Enable)
			{
				for (auto ItInner = It.Value().NamesInThisGroup.CreateConstIterator(); ItInner; ++ItInner)
				{
					EnableStat(ItInner.Key(), ItInner.Value());
				}
			}
			else
			{
				for (auto ItInner = It.Value().NamesInThisGroup.CreateConstIterator(); ItInner; ++ItInner)
				{
					DisableStat(ItInner.Value());
				}
			}
		}
		FThreadStats::PrimaryDisableChangeTagLockSubtract();
	}
	virtual void ResetHighPerformanceEnableForAllGroups() override
	{
		UE::TScopeLock ScopeLock(SynchronizationObject);
		FThreadStats::PrimaryDisableChangeTagLockAdd();
		for (auto It = HighPerformanceEnable.CreateIterator(); It; ++It)
		{
			It.Value().CurrentEnable = It.Value().DefaultEnable;
			if (It.Value().DefaultEnable)
			{
				for (auto ItInner = It.Value().NamesInThisGroup.CreateConstIterator(); ItInner; ++ItInner)
				{
					EnableStat(ItInner.Key(), ItInner.Value());
				}
			}
			else
			{
				for (auto ItInner = It.Value().NamesInThisGroup.CreateConstIterator(); ItInner; ++ItInner)
				{
					DisableStat(ItInner.Value());
				}
			}
		}
		FThreadStats::PrimaryDisableChangeTagLockSubtract();
	}

	virtual TStatId GetHighPerformanceEnableForStat(FName StatShortName, const char* InGroup, const char* InCategory, bool bDefaultEnable, bool bShouldClearEveryFrame, EStatDataType::Type InStatType, TCHAR const* InDescription, bool bCycleStat, bool bSortByName, FPlatformMemory::EMemoryCounterRegion MemoryRegion = FPlatformMemory::MCR_Invalid) override
	{
		LLM_SCOPE(ELLMTag::Stats);

		FName Stat;
		TStatIdData* Result;
		FString StatDescription;
		{
			UE::TScopeLock ScopeLock(SynchronizationObject);

			FStatNameAndInfo LongName(StatShortName, InGroup, InCategory, InDescription, InStatType, bShouldClearEveryFrame, bCycleStat, bSortByName, MemoryRegion);

			Stat = LongName.GetEncodedName();

			FName Group(InGroup);
			FGroupEnable* Found = HighPerformanceEnable.Find(Group);
			if (Found)
			{
				if (Found->DefaultEnable != bDefaultEnable)
				{
					UE_LOGF(LogStatGroupEnableManager, Fatal, "Stat group %ls was was defined both on and off by default.", *Group.ToString());
				}
				TStatIdData** StatFound = Found->NamesInThisGroup.Find(Stat);
				TStatIdData** StatFoundAlways = Found->AlwaysEnabledNamesInThisGroup.Find(Stat);
				if (StatFound)
				{
					if (StatFoundAlways)
					{
						UE_LOGF(LogStatGroupEnableManager, Fatal, "Stat %ls is both always enabled and not always enabled, so it was used for two different things.", *Stat.ToString());
					}
					return TStatId(*StatFound);
				}
				else if (StatFoundAlways)
				{
					return TStatId(*StatFoundAlways);
				}
			}
			else
			{
				Found = &HighPerformanceEnable.Add(Group, FGroupEnable(bDefaultEnable || !bShouldClearEveryFrame));

				// this was set up before we saw the group, so set the enable now
				if (EnableForNewGroup.Contains(Group))
				{
					Found->CurrentEnable = EnableForNewGroup.FindChecked(Group);
					EnableForNewGroup.Remove(Group); // by definition, we will never need this again
				}
				else if (UseEnableForNewGroups)
				{
					Found->CurrentEnable = EnableForNewGroups;
				}
			}
			if (StatIDs.Num() == 0 || StatIDs.Last().Num() == NUM_PER_BLOCK)
			{
				TArray<TStatIdData>& NewBlock = StatIDs.AddDefaulted_GetRef();
				NewBlock.Reserve(NUM_PER_BLOCK);
			}

			if (Found->CurrentEnable)
			{
				Result = &StatIDs.Last().Emplace_GetRef(NameToMinimalName(Stat));
			}
			else
			{
				Result = &StatIDs.Last().AddDefaulted_GetRef();
			}

			StatDescription = InDescription ? InDescription : StatShortName.GetPlainNameString();

			const int32 StatDescLen = StatDescription.Len() + 1;

			int32 MemoryAllocated = 0;

			// Get the wide stat description.
			{
				auto StatDescriptionWide = StringCast<WIDECHAR>(*StatDescription, StatDescLen);
				int32 StatDescriptionWideLength = StatDescriptionWide.Length();
				TUniquePtr<WIDECHAR[]> StatDescWide = MakeUnique<WIDECHAR[]>(StatDescriptionWideLength + 1);
				TCString<WIDECHAR>::Strncpy(StatDescWide.Get(), StatDescriptionWide.Get(), StatDescriptionWideLength + 1);
				Result->StatDescriptionWide = MoveTemp(StatDescWide);

				MemoryAllocated += StatDescriptionWideLength * sizeof(WIDECHAR);
			}

			// Get the ansi stat description.
			{
				auto StatDescriptionAnsi = StringCast<ANSICHAR>(*StatDescription, StatDescLen);
				int32 StatDescriptionAnsiLength = StatDescriptionAnsi.Length();
				TUniquePtr<ANSICHAR[]> StatDescAnsi = MakeUnique<ANSICHAR[]>(StatDescriptionAnsiLength + 1);
				TCString<ANSICHAR>::Strncpy(StatDescAnsi.Get(), StatDescriptionAnsi.Get(), StatDescriptionAnsiLength + 1);
				Result->StatDescriptionAnsi = MoveTemp(StatDescAnsi);

				MemoryAllocated += StatDescriptionAnsiLength * sizeof(ANSICHAR);
			}

			UE_AUTORTFM_ONCOMMIT(this, MemoryAllocated)
			{
				// Safe to defer this because FStatGroupEnableManager is a singleton, see IStatGroupEnableManager::Get()
				MemoryCounter.Add(MemoryAllocated);
			};

			if (bShouldClearEveryFrame)
			{
				Found->NamesInThisGroup.Add(Stat, Result);
			}
			else
			{
				Found->AlwaysEnabledNamesInThisGroup.Add(Stat, Result);
			}
		}

#if STATSTRACE_ENABLED
		if (!bCycleStat && (InStatType == EStatDataType::ST_int64 || InStatType == EStatDataType::ST_double))
		{
			if (!StatShortName.GetDisplayNameEntry()->IsWide())
			{
				ANSICHAR NameBuffer[1024];
				StatShortName.GetPlainANSIString(NameBuffer);
				FStatsTrace::DeclareStat(Stat, NameBuffer, *StatDescription, InGroup, InStatType == EStatDataType::ST_double, MemoryRegion != FPlatformMemory::MCR_Invalid, bShouldClearEveryFrame);
			}
			else
			{
				FString StatShortNameString = StatShortName.GetPlainNameString();
				FStatsTrace::DeclareStat(Stat, TCHAR_TO_ANSI(*StatShortNameString), *StatDescription, InGroup, InStatType == EStatDataType::ST_double, MemoryRegion != FPlatformMemory::MCR_Invalid, bShouldClearEveryFrame);
			}
		}
#endif
		return TStatId(Result);
	}

	void ListGroup(FName Group)
	{
		FGroupEnable* Found = HighPerformanceEnable.Find(Group);
		if (Found)
		{
			UE_LOGF(LogStatGroupEnableManager, Display, "  %d  default %d %ls", !!Found->CurrentEnable, !!Found->DefaultEnable, *Group.ToString());
		}
	}

	void ListGroups(bool bDetailed = false)
	{
		for (auto It = HighPerformanceEnable.CreateConstIterator(); It; ++It)
		{
			UE_LOGF(LogStatGroupEnableManager, Display, "  %d  default %d %ls", !!It.Value().CurrentEnable, !!It.Value().DefaultEnable, *(It.Key().ToString()));
			if (bDetailed)
			{
				for (auto ItInner = It.Value().NamesInThisGroup.CreateConstIterator(); ItInner; ++ItInner)
				{
					UE_LOGF(LogStatGroupEnableManager, Display, "      %d %ls", !ItInner.Value()->IsNone(), *ItInner.Key().ToString());
				}
				for (auto ItInner = It.Value().AlwaysEnabledNamesInThisGroup.CreateConstIterator(); ItInner; ++ItInner)
				{
					UE_LOGF(LogStatGroupEnableManager, Display, "      (always enabled) %ls", *ItInner.Key().ToString());
				}
			}
		}
	}

	FName CheckGroup(TCHAR const*& Cmd, bool Enable)
	{
		FString MaybeGroup;
		FParse::Token(Cmd, MaybeGroup, false);
		MaybeGroup = FString("STATGROUP_") + MaybeGroup;
		FName MaybeGroupFName(*MaybeGroup);

		FGroupEnable* Found = HighPerformanceEnable.Find(MaybeGroupFName);
		if (!Found)
		{
			EnableForNewGroup.Add(MaybeGroupFName, Enable);
			ListGroups();
			UE_LOGF(LogStatGroupEnableManager, Display, "Group Not Found %ls", *MaybeGroupFName.ToString());
			return NAME_None;
		}
		SetHighPerformanceEnableForGroup(MaybeGroupFName, Enable);
		ListGroup(MaybeGroupFName);
		return MaybeGroupFName;
	}

	void StatGroupEnableManagerCommand(FString const& InCmd) override
	{
		UE::TScopeLock ScopeLock(SynchronizationObject);
		const TCHAR* Cmd = *InCmd;
		if (FParse::Command(&Cmd, TEXT("list")))
		{
			ListGroups();
		}
		else if (FParse::Command(&Cmd, TEXT("listall")))
		{
			ListGroups(true);
		}
		else if (FParse::Command(&Cmd, TEXT("enable")))
		{
			CheckGroup(Cmd, true);
		}
		else if (FParse::Command(&Cmd, TEXT("disable")))
		{
			CheckGroup(Cmd, false);
		}
		else if (FParse::Command(&Cmd, TEXT("none")))
		{
			EnableForNewGroups = false;
			UseEnableForNewGroups = true;
			SetHighPerformanceEnableForAllGroups(false);
			ListGroups();
		}
		else if (FParse::Command(&Cmd, TEXT("all")))
		{
			EnableForNewGroups = true;
			UseEnableForNewGroups = true;
			SetHighPerformanceEnableForAllGroups(true);
			ListGroups();
		}
		else if (FParse::Command(&Cmd, TEXT("default")))
		{
			UseEnableForNewGroups = false;
			EnableForNewGroup.Empty();
			ResetHighPerformanceEnableForAllGroups();
			ListGroups();
		}
	}
};

IStatGroupEnableManager& IStatGroupEnableManager::Get()
{
	static FStatGroupEnableManager Singleton;
	return Singleton;
}

/*-----------------------------------------------------------------------------
	FStatNameAndInfo
-----------------------------------------------------------------------------*/

CORE_API const char* FStatNameAndInfo::GpuStatCategory = "STATCAT_GPU";

FName FStatNameAndInfo::ToLongName(FName InStatName, char const* InGroup, char const* InCategory, TCHAR const* InDescription, bool InSortByName)
{
	TStringBuilder<256> LongName;
	if (InGroup)
	{
		LongName += TEXT("//");
		LongName += InGroup;
		LongName += TEXT("//");
	}
	InStatName.AppendString(LongName);
	if (InDescription)
	{
		LongName += TEXT("///");
		FStatsUtils::ToEscapedString(InDescription, LongName);
		LongName += TEXT("///");
	}
	if (InCategory)
	{
		LongName += TEXT("####");
		LongName += InCategory;
		LongName += TEXT("####");
	}
	if (InSortByName)
	{
		LongName += TEXT("/#/#");
		LongName += TEXT("SortByName");
		LongName += TEXT("/#/#");
	}
	return FName(*LongName);
}

FName FStatNameAndInfo::GetShortNameFrom(FName InLongName)
{
	TStringBuilder<256> Temp;
	Temp << InLongName;
	FStringView Input(Temp);

	if (Input.StartsWith(TEXT("//"), ESearchCase::CaseSensitive))
	{
		Input.RemovePrefix(2);
		const int32 IndexEnd = UE::String::FindFirst(Input, TEXT("//"));
		if (IndexEnd == INDEX_NONE)
		{
			checkStats(0);
			return InLongName;
		}
		Input.RightChopInline(IndexEnd + 2);
	}
	const int32 DescIndexEnd = UE::String::FindFirst(Input, TEXT("///"), ESearchCase::CaseSensitive);
	if (DescIndexEnd != INDEX_NONE)
	{
		Input.LeftInline(DescIndexEnd);
	}
	const int32 CategoryIndexEnd = UE::String::FindFirst(Input, TEXT("####"), ESearchCase::CaseSensitive);
	if (DescIndexEnd == INDEX_NONE && CategoryIndexEnd != INDEX_NONE)
	{
		Input.LeftInline(CategoryIndexEnd);
	}
	const int32 SortByNameIndexEnd = UE::String::FindFirst(Input, TEXT("/#/#"), ESearchCase::CaseSensitive);
	if (DescIndexEnd == INDEX_NONE && CategoryIndexEnd == INDEX_NONE && SortByNameIndexEnd != INDEX_NONE)
	{
		Input.LeftInline(SortByNameIndexEnd);
	}
	return FName(Input);
}

FName FStatNameAndInfo::GetGroupNameFrom(FName InLongName)
{
	TStringBuilder<256> Temp;
	Temp << InLongName;
	FStringView Input(Temp);

	if (Input.StartsWith(TEXT("//"), ESearchCase::CaseSensitive))
	{
		Input.RemovePrefix(2);
		if (Input.StartsWith(TEXT("Groups//")))
		{
			Input.RemovePrefix(8);
		}
		const int32 IndexEnd = UE::String::FindFirst(Input, TEXT("//"));
		if (IndexEnd != INDEX_NONE)
		{
			return FName(Input.Left(IndexEnd));
		}
		checkStats(0);
	}
	return FName();
}

FString FStatNameAndInfo::GetDescriptionFrom(FName InLongName)
{
	FString Input(InLongName.ToString());

	const int32 IndexStart = Input.Find(TEXT("///"), ESearchCase::CaseSensitive);
	if (IndexStart != INDEX_NONE)
	{
		Input.RightChopInline(IndexStart + 3, EAllowShrinking::No);
		const int32 IndexEnd = Input.Find(TEXT("///"), ESearchCase::CaseSensitive);
		if (IndexEnd != INDEX_NONE)
		{
			return FStatsUtils::FromEscapedString(*Input.Left(IndexEnd));
		}
	}
	return FString();
}

FName FStatNameAndInfo::GetGroupCategoryFrom(FName InLongName)
{
	FString Input(InLongName.ToString());

	const int32 IndexStart = Input.Find(TEXT("####"), ESearchCase::CaseSensitive);
	if (IndexStart != INDEX_NONE)
	{
		Input.RightChopInline(IndexStart + 4, EAllowShrinking::No);
		const int32 IndexEnd = Input.Find(TEXT("####"), ESearchCase::CaseSensitive);
		if (IndexEnd != INDEX_NONE)
		{
			return FName(*Input.Left(IndexEnd));
		}
		checkStats(0);
	}
	return NAME_None;
}

bool FStatNameAndInfo::GetSortByNameFrom(FName InLongName)
{
	FString Input(InLongName.ToString());

	const int32 IndexStart = Input.Find(TEXT("/#/#"), ESearchCase::CaseSensitive);
	if (IndexStart != INDEX_NONE)
	{
		Input.RightChopInline(IndexStart + 4, EAllowShrinking::No);
		const int32 IndexEnd = Input.Find(TEXT("/#/#"), ESearchCase::CaseSensitive);
		if (IndexEnd != INDEX_NONE)
		{
			return Input.Left(IndexEnd) == TEXT("SortByName");
		}
		checkStats(0);
	}
	return false;
}

/*-----------------------------------------------------------------------------
	FStatsThread
-----------------------------------------------------------------------------*/

class FLastFrameBarrier
{
public:
	void Reset();
	void Wait();

private:
	UE::Tasks::FTask Barrier;
	FGraphEventRef Event;
};

void FLastFrameBarrier::Reset()
{
	Event = {};
	Barrier = {};
}

void FLastFrameBarrier::Wait()
{
	if (FPlatformProcess::SupportsMultithreading())
	{
		Barrier.Wait();
		Barrier = GStatsPipe.Launch(UE_SOURCE_LOCATION, [] {});
	}
	else
	{
		if (Event.GetReference())
		{
			FTaskGraphInterface::Get().WaitUntilTaskCompletes(Event, ENamedThreads::GameThread_Local);
		}

		Event = TGraphTask<FNullGraphTask>::CreateTask(NULL, ENamedThreads::GameThread).
			ConstructAndDispatchWhenReady(TStatId{}, ENamedThreads::GameThread);
	}

}

static int32				CurrentEventIndex = 0;
static FLastFrameBarrier	LastFrameBarriers[MAX_STAT_LAG];

static TAutoConsoleVariable<int32> CVarDumpStatPackets(TEXT("DumpStatPackets"), 0, TEXT("If true, dump stat packets."));

/** The rendering thread runnable object. */
class FStatsThread
{
	/** Array of stat packets, queued data to be processed on this thread. */
	FStatPacketArray IncomingData;

	/** Stats state. */
	FStatsThreadState& State;

	/** Whether we are ready to process the packets, sets by game or render packets. */
	bool bReadyToProcess;

public:
	static FStatsThread& Get()
	{
		static FStatsThread Singleton;
		return Singleton;
	}

	/** Default constructor. */
	FStatsThread()
		: State(FStatsThreadState::GetLocalState())
		, bReadyToProcess(false)
	{
	}

	/** Received a stat packet from other thread and add to the processing queue. */
	void StatMessage(FStatPacket* Packet)
	{
		LLM_SCOPE(ELLMTag::Stats);

		if (CVarDumpStatPackets.GetValueOnAnyThread())
		{
			UE_LOGF(LogStats, Log, "Packet from %x with %d messages", Packet->ThreadId, Packet->StatMessages.Num());
		}

		bReadyToProcess = Packet->ThreadType != EThreadType::Other;
		IncomingData.Packets.Add(Packet);
		State.NumStatMessages.Add(Packet->StatMessages.Num());

		Process();
	}

private:
	void Process()
	{
		static double LastTime = -1.0;
		bool bShouldProcess = false;

		const int32 MaxIncomingPackets = 16;
		if (FThreadStats::bIsRawStatsActive)
		{
			// For raw stats we process every 24MB of packet data to minimize the stats messages memory usage.
			//const bool bShouldProcessRawStats = IncomingData.Packets.Num() > 10;
			const int32 MaxIncomingMessages = 24 * 1024 * 1024 / sizeof(FStatMessage);

			int32 IncomingDataMessages = 0;
			for (FStatPacket* Packet : IncomingData.Packets)
			{
				IncomingDataMessages += Packet->StatMessages.Num();
			}

			bShouldProcess = IncomingDataMessages > MaxIncomingMessages || IncomingData.Packets.Num() > MaxIncomingPackets;
		}
		else
		{
			// For regular stats we won't process more than every 5ms or every 16 packets.
			// Commandlet stats are flushed as soon as.
			bShouldProcess = bReadyToProcess && (FPlatformTime::Seconds() - LastTime > 0.005f || IncomingData.Packets.Num() > MaxIncomingPackets || UE::Stats::FStats::EnabledForCommandlet());
		}

		if (bShouldProcess)
		{
			SCOPE_CYCLE_COUNTER(STAT_StatsNewTick);

			IStatGroupEnableManager::Get().UpdateMemoryUsage();
			State.UpdateStatMessagesMemoryUsage();

			bReadyToProcess = false;
			FStatPacketArray NowData;
			Exchange(NowData.Packets, IncomingData.Packets);
			INC_DWORD_STAT_BY(STAT_StatFramePacketsRecv, NowData.Packets.Num());
			{
				SCOPE_CYCLE_COUNTER(STAT_StatsNewParseMeta);
				TArray64<FStatMessage> MetaMessages;
				{
					UE::TScopeLock Lock(FStartupMessages::Get().CriticalSection);
					Exchange(FStartupMessages::Get().DelayedMessages, MetaMessages);
				}
				if (MetaMessages.Num())
				{
					State.ProcessMetaDataOnly(MetaMessages);
				}
			}
			{
				SCOPE_CYCLE_COUNTER(STAT_ScanForAdvance);
				State.ScanForAdvance(NowData);
			}

			if (FThreadStats::bIsRawStatsActive)
			{
				// Process raw stats.
				State.ProcessRawStats(NowData);
				State.ResetRegularStats();
			}
			else
			{
				// Process regular stats.
				SCOPE_CYCLE_COUNTER(STAT_StatsNewAddToHistory);
				State.ResetRawStats();
				State.AddToHistoryAndEmpty(NowData);
			}
			check(!NowData.Packets.Num());
			LastTime = FPlatformTime::Seconds();
		}
	}
};

/*-----------------------------------------------------------------------------
	FThreadStatsPool
-----------------------------------------------------------------------------*/

FThreadStatsPool::FThreadStatsPool()
{
}

FThreadStatsPool& FThreadStatsPool::Get()
{
	static FThreadStatsPool Singleton;
	return Singleton;
}

FThreadStats* FThreadStatsPool::GetFromPool()
{
	LLM_SCOPE(ELLMTag::Stats);

	if (void* Address = Pool.Pop())
	{
		return new (Address) FThreadStats;
	}

	return new FThreadStats;
}

void FThreadStatsPool::ReturnToPool(FThreadStats* Instance)
{
	check(Instance);
	Instance->~FThreadStats();
	Pool.Push(Instance);
}

/*-----------------------------------------------------------------------------
	FThreadStats
-----------------------------------------------------------------------------*/

uint32 FThreadStats::TlsSlot = FPlatformTLS::InvalidTlsSlot;
FThreadSafeCounter FThreadStats::PrimaryEnableCounter;
FThreadSafeCounter FThreadStats::PrimaryEnableUpdateNumber;
FThreadSafeCounter FThreadStats::PrimaryDisableChangeTagLock;
bool FThreadStats::bPrimaryEnable = false;
bool FThreadStats::bPrimaryDisableForever = false;
bool FThreadStats::bIsRawStatsActive = false;
FEndOfPipeStats FEndOfPipeStats::EndOfPipeStats;

FThreadStats::FThreadStats(EThreadType::Type InThreadType)
	: Packet(InThreadType)
	, CurrentGameFrame(UE::Stats::FStats::GameThreadStatsFrame.Load(EMemoryOrder::Relaxed))
{
	if (Packet.ThreadType == EThreadType::Invalid)
	{
		Packet.SetThreadProperties();
	}

	if (Packet.ThreadType != EThreadType::EndOfPipe)
	{
		check(FPlatformTLS::IsValidTlsSlot(TlsSlot));
		FPlatformTLS::SetTlsValue(TlsSlot, this);
	}
}

void FThreadStats::CheckEnable()
{
	bool bNewPrimaryEnable = 
		   WillEverCollectData()
		&& (!IsRunningCommandlet() || UE::Stats::FStats::EnabledForCommandlet())
		&& IsThreadingReady()
		&& PrimaryEnableCounter.GetValue();

	if (bPrimaryEnable != bNewPrimaryEnable)
	{
		PrimaryDisableChangeTagLockAdd();
		bPrimaryEnable = bNewPrimaryEnable;
		PrimaryDisableChangeTagLockSubtract();
	}
}

bool FThreadStats::DetectAndUpdateCurrentGameFrame()
{
	if (Packet.ThreadType == EThreadType::Other)
	{
		FPlatformMisc::MemoryBarrier();
		int32 Frame = UE::Stats::FStats::GameThreadStatsFrame;
		const bool bFrameHasChanged = Frame > CurrentGameFrame;
		if (bFrameHasChanged)
		{
			CurrentGameFrame = Frame;
			Packet.AssignFrame(CurrentGameFrame);
			return true;
		}
	}

	return false;
}

void FThreadStats::Flush(bool bHasBrokenCallstacks /*= false*/, bool bForceFlush /*= false*/)
{
	if (bPrimaryDisableForever)
	{
		Packet.StatMessages.Empty();
		return;
	}

	if (bIsRawStatsActive)
	{
		FlushRawStats(bHasBrokenCallstacks, bForceFlush);
	}
	else
	{
		FlushRegularStats(bHasBrokenCallstacks, bForceFlush);
	}
}

void FThreadStats::SendMessage_Async(FStatPacket* ToSend)
{
	if (FPlatformProcess::SupportsMultithreading())
	{
		GStatsPipe.Launch(UE_SOURCE_LOCATION, [ToSend] { FStatsThread::Get().StatMessage(ToSend); });
	}
	else
	{
		FFunctionGraphTask::CreateAndDispatchWhenReady([ToSend] { FStatsThread::Get().StatMessage(ToSend); }, TStatId{}, nullptr, ENamedThreads::GameThread);
	}
}

void FThreadStats::FlushRegularStats(bool bHasBrokenCallstacks, bool bForceFlush)
{
	LLM_SCOPE(ELLMTag::Stats);

	if (bReentranceGuard)
	{
		return;
	}
	TGuardValue<bool> Guard(bReentranceGuard, true);

	enum
	{
		PRESIZE_MAX_NUM_ENTRIES = 10,
		PRESIZE_MAX_SIZE = 256 * 1024,
	};


	// Sends all collected messages when:
	// The current game frame has changed.
	// This a force flush when we shutting down the thread stats.
	// This is an explicit flush from the game thread or the render thread.
	const bool bFrameHasChanged = DetectAndUpdateCurrentGameFrame();
	const bool bSendStatPacket = bFrameHasChanged || bForceFlush || bSawExplicitFlush;
	if (!bSendStatPacket)
	{
		return;
	}

	if ((!ScopeCount || bForceFlush) && Packet.StatMessages.Num())
	{
		if (Packet.StatMessagesPresize.Num() >= PRESIZE_MAX_NUM_ENTRIES)
		{
			Packet.StatMessagesPresize.RemoveAt(0);
		}
		if (Packet.StatMessages.Num() < PRESIZE_MAX_SIZE)
		{
			Packet.StatMessagesPresize.Add(Packet.StatMessages.Num());
		}
		else
		{
			UE_LOGF(LogStats, Verbose, "StatMessage Packet has more than %i messages.  Ignoring for the presize history.", (int32)PRESIZE_MAX_SIZE);
		}
		FStatPacket* ToSend = new FStatPacket(Packet);
		Exchange(ToSend->StatMessages, Packet.StatMessages);
		ToSend->bBrokenCallstacks = bHasBrokenCallstacks;

		check(!Packet.StatMessages.Num());
		if (Packet.StatMessagesPresize.Num() > 0)
		{
			int32 MaxPresize = Packet.StatMessagesPresize[0];
			for (int32 Index = 0; Index < Packet.StatMessagesPresize.Num(); ++Index)
			{
				if (MaxPresize < Packet.StatMessagesPresize[Index])
				{
					MaxPresize = Packet.StatMessagesPresize[Index];
				}
			}
			Packet.StatMessages.Empty(MaxPresize);
		}

		SendMessage_Async(ToSend);
		UpdateExplicitFlush();
	}
}

void FThreadStats::FlushRawStats(bool bHasBrokenCallstacks /*= false*/, bool bForceFlush /*= false*/)
{
	LLM_SCOPE(ELLMTag::Stats);

	if (bReentranceGuard)
	{
		return;
	}
	TGuardValue<bool> Guard(bReentranceGuard, true);

	enum
	{
		/** Maximum number of messages in the stat packet. */
		MAX_RAW_MESSAGES_IN_PACKET = 1024 * 1024 / sizeof(FStatMessage),
	};

	// Sends all collected messages when:
	// Number of messages is greater than MAX_RAW_MESSAGES_IN_PACKET.
	// The current game frame has changed.
	// This a force flush when we shutting down the thread stats.
	// This is an explicit flush from the game thread or the render thread.
	const bool bFrameHasChanged = DetectAndUpdateCurrentGameFrame();
	const int32 NumMessages = Packet.StatMessages.Num();
	if (NumMessages > MAX_RAW_MESSAGES_IN_PACKET || bFrameHasChanged || bForceFlush || bSawExplicitFlush)
	{
		SCOPE_CYCLE_COUNTER(STAT_FlushRawStats);

		FStatPacket* ToSend = new FStatPacket(Packet);
		Exchange(ToSend->StatMessages, Packet.StatMessages);
		ToSend->bBrokenCallstacks = bHasBrokenCallstacks;

		check(!Packet.StatMessages.Num());

		SendMessage_Async(ToSend);
		UpdateExplicitFlush();

		const float NumMessagesAsMB = float(NumMessages * sizeof(FStatMessage)) / 1024.0f / 1024.0f;
		if (NumMessages > 524288)
		{
			UE_LOGF(LogStats, Warning, "FlushRawStats NumMessages: %i (%.2f MB), Thread: %u", NumMessages, NumMessagesAsMB, Packet.ThreadId);
		}

		UE_LOGF(LogStats, Verbose, "FlushRawStats NumMessages: %i (%.2f MB), Thread: %u", NumMessages, NumMessagesAsMB, Packet.ThreadId);
	}
}

void FThreadStats::CheckForCollectingStartupStats()
{
	FString CmdLine(FCommandLine::Get());
	FString StatCmds("-StatCmds=");
	while (1)
	{
		FString Cmds;
		if (!FParse::Value(*CmdLine, *StatCmds, Cmds, false))
		{
			break;
		}
		TArray<FString> CmdsArray;
		Cmds.ParseIntoArray(CmdsArray, TEXT(","), true);
		for (int32 Index = 0; Index < CmdsArray.Num(); Index++)
		{
			CmdsArray[Index].TrimStartInline();
			FString StatCmd = FString("stat ") + CmdsArray[Index];
			UE_LOGF(LogStatGroupEnableManager, Log, "Sending Stat Command '%ls'", *StatCmd);
			UE::Stats::DirectStatsCommand(*StatCmd);
		}
		int32 Index = CmdLine.Find(*StatCmds);
		ensure(Index >= 0);
		if (Index == INDEX_NONE)
		{
			break;
		}
		CmdLine.MidInline(Index + StatCmds.Len(), MAX_int32, EAllowShrinking::No);
	}

	if (FParse::Param(FCommandLine::Get(), TEXT("LoadTimeStats")))
	{
		UE::Stats::DirectStatsCommand(TEXT("stat group enable LinkerLoad"));
		UE::Stats::DirectStatsCommand(TEXT("stat group enable AsyncLoad"));
		UE::Stats::DirectStatsCommand(TEXT("stat group enable LoadTimeVerbose"));
		UE::Stats::DirectStatsCommand(TEXT("stat dumpsum -start -ms=250 -num=240"), true);
	}
	else if (FParse::Param(FCommandLine::Get(), TEXT("LoadTimeFile")) || UE::Stats::FStats::HasLoadTimeFileForCommandletToken())
	{
		UE::Stats::DirectStatsCommand(TEXT("stat group enable LinkerLoad"));
		UE::Stats::DirectStatsCommand(TEXT("stat group enable AsyncLoad"));
		UE::Stats::DirectStatsCommand(TEXT("stat group enable LoadTimeVerbose"));
		UE::Stats::DirectStatsCommand(TEXT("stat startfile"), true);
	}
	else if (UE::Stats::FStats::HasLoadTimeStatsForCommandletToken())
	{
		UE::Stats::DirectStatsCommand(TEXT("stat group enable LinkerLoad"));
		UE::Stats::DirectStatsCommand(TEXT("stat group enable AsyncLoad"));
		UE::Stats::DirectStatsCommand(TEXT("stat group enable LoadTimeVerbose"));
		UE::Stats::DirectStatsCommand(TEXT("stat dumpsum -start"), true);
	}

	STAT_ADD_CUSTOMMESSAGE_NAME(STAT_NamedMarker, TEXT("CheckForCollectingStartupStats"));
}

void FThreadStats::ExplicitFlush(bool DiscardCallstack)
{
	FThreadStats* ThreadStats = GetThreadStats();
	//check(ThreadStats->Packet.ThreadType != EThreadType::Other);
	if (ThreadStats->bWaitForExplicitFlush)
	{
		ThreadStats->ScopeCount--; // the main thread pre-incremented this to prevent stats from being sent. we send them at the next available opportunity
		ThreadStats->bWaitForExplicitFlush = 0;
	}
	bool bHasBrokenCallstacks = false;
	if (DiscardCallstack && ThreadStats->ScopeCount)
	{
		ThreadStats->ScopeCount = 0;
		bHasBrokenCallstacks = true;
	}
	ThreadStats->bSawExplicitFlush = true;
	ThreadStats->Flush(bHasBrokenCallstacks);
}

void FThreadStats::StartThread()
{
	AutoRTFM::UnreachableIfClosed("TODO: Can remove if `FEngineLoop` is marked `AUTORTFM_DISABLE`!");

	FThreadStats::FrameDataIsIncomplete(); // make this non-zero
	check(IsInGameThread());
	check(!IsThreadingReady());
	// Preallocate a bunch of FThreadStats to avoid dynamic memory allocation.
	// (Must do this before we expose ourselves to other threads via tls).
	FThreadStatsPool::Get();
	FStatsThreadState::GetLocalState(); // start up the state
	if (!FPlatformTLS::IsValidTlsSlot(TlsSlot))
	{
		TlsSlot = FPlatformTLS::AllocTlsSlot();
		check(FPlatformTLS::IsValidTlsSlot(TlsSlot));
	}

	check(IsThreadingReady());
	CheckEnable();

	if (FThreadStats::WillEverCollectData())
	{
		FThreadStats::ExplicitFlush(); // flush the stats and set update the scope so we don't flush again until a frame update, this helps prevent fragmentation
	}
	FStartupMessages::Get().AddThreadMetadata(NAME_GameThread, FPlatformTLS::GetCurrentThreadId());

	CheckForCollectingStartupStats();

	UE_LOGF(LogStats, Log, "Stats thread started at %f", FPlatformTime::Seconds() - GStartTime);
}

void FThreadStats::StopThread()
{
	// Nothing to stop if it was never started
	if (IsThreadingReady())
	{
		if (UE::Stats::FStats::HasLoadTimeStatsForCommandletToken())
		{
			// Dump all the collected stats to the log, if any.
			UE::Stats::DirectStatsCommand(TEXT("stat dumpsum -stop -ms=100"), true);
		}

		// If we are writing stats data, stop it now.
		UE::Stats::DirectStatsCommand(TEXT("stat stopfile"), true);

		FThreadStats::PrimaryDisableForever();

		WaitForStats();
	}

	// wait for the pipe to complete all currently piped tasks
	while (GStatsPipe.HasWork())
	{
		FPlatformProcess::Yield();
	}

	// release memory before the allocator is destroyed
	for (FLastFrameBarrier& Barrier : LastFrameBarriers)
	{
		Barrier.Reset();
	}
}

void FThreadStats::WaitForStats()
{
	if (FPlatformProcess::SkipWaitForStats())
	{
		return;
	}

	check(IsInGameThread());
	if (IsThreadingReady() && !bPrimaryDisableForever)
	{
		{
			DECLARE_CYCLE_STAT(TEXT("WaitForStats"), STAT_WaitForStats, STATGROUP_Engine);
			SCOPE_CYCLE_COUNTER(STAT_WaitForStats);
			int32 EventIndex = (CurrentEventIndex + MAX_STAT_LAG - 1) % MAX_STAT_LAG;
			LastFrameBarriers[EventIndex].Wait();
			CurrentEventIndex++;
		}

#if	!UE_BUILD_SHIPPING
		DebugLeakTest();
#endif // !UE_BUILD_SHIPPING
	}
}

#endif // STATS