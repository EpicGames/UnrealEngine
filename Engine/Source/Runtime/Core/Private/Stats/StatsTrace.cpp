// Copyright Epic Games, Inc. All Rights Reserved.
#include "Stats/StatsTrace.h"
#include "ProfilingDebugging/MiscTrace.h"
#include "Templates/Atomic.h"
#include "Misc/CString.h"
#include "HAL/LowLevelMemTracker.h"
#include "HAL/PlatformTime.h"
#include "HAL/PlatformTLS.h"
#include "Trace/Trace.inl"
#include "UObject/NameTypes.h"

#if STATSTRACE_ENABLED

UE_TRACE_CHANNEL(StatsChannel, "Named stat declarations and batched stat operations (increment, decrement, add, set) for integer and float stats declared with DECLARE_STATS_GROUP. Viewable in Unreal Insights alongside the Counters channel.")

UE_TRACE_EVENT_BEGIN(Stats, Spec, NoSync|Important)
	UE_TRACE_EVENT_FIELD(uint32, Id)
	UE_TRACE_EVENT_FIELD(bool, IsFloatingPoint)
	UE_TRACE_EVENT_FIELD(bool, IsMemory)
	UE_TRACE_EVENT_FIELD(bool, ShouldClearEveryFrame)
	UE_TRACE_EVENT_FIELD(UE::Trace::AnsiString, Name)
	UE_TRACE_EVENT_FIELD(UE::Trace::AnsiString, Description)
	UE_TRACE_EVENT_FIELD(UE::Trace::AnsiString, Group)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Stats, EventBatch2)
	UE_TRACE_EVENT_FIELD(uint8[], Data)
UE_TRACE_EVENT_END()

struct FStatsTraceInternal
{
public:
	enum
	{
		MaxBufferSize = 512,
		MaxEncodedEventSize = 30, // 10 + 10 + 10
		FullBufferThreshold = MaxBufferSize - MaxEncodedEventSize,
	};

	enum EOpType
	{
		Increment = 0,
		Decrement = 1,
		AddInteger = 2,
		SetInteger = 3,
		AddFloat = 4,
		SetFloat = 5,
	};

	struct FThreadState
	{
		uint64 LastCycle;
		uint16 BufferSize;
		uint8 Buffer[MaxBufferSize];
	};

	static FThreadState* GetThreadState() { return ThreadLocalThreadState; }
	FORCENOINLINE static FThreadState* InitThreadState();
	FORCENOINLINE static void FlushThreadBuffer(FThreadState* ThreadState);
	static void BeginEncodeOp(const FName& Stat, EOpType Op, FStatsTraceInternal::FThreadState*& OutThreadState, uint8*& OutBufferPtr);
	static void EndEncodeOp(FStatsTraceInternal::FThreadState* ThreadState, uint8* BufferPtr);

private:
	static thread_local FThreadState* ThreadLocalThreadState;
};

thread_local FStatsTraceInternal::FThreadState* FStatsTraceInternal::ThreadLocalThreadState = nullptr;

FStatsTraceInternal::FThreadState* FStatsTraceInternal::InitThreadState()
{
	LLM_SCOPE_BYNAME(TEXT("Trace/Stats"));
	ThreadLocalThreadState = new FThreadState();
	ThreadLocalThreadState->LastCycle = 0;
	ThreadLocalThreadState->BufferSize = 0;
	return ThreadLocalThreadState;
}

void FStatsTraceInternal::FlushThreadBuffer(FThreadState* ThreadState)
{
	UE_TRACE_LOG(Stats, EventBatch2, StatsChannel)
		<< EventBatch2.Data(ThreadState->Buffer, ThreadState->BufferSize);

	ThreadState->LastCycle = 0; // each batch starts with an absolute timestamp value
	ThreadState->BufferSize = 0;
}

void FStatsTraceInternal::BeginEncodeOp(const FName& Stat, EOpType Op, FThreadState*& OutThreadState, uint8*& OutBufferPtr)
{
	uint64 Cycle = FPlatformTime::Cycles64();
	OutThreadState = GetThreadState();
	if (!OutThreadState)
	{
		OutThreadState = InitThreadState();
	}
	if (OutThreadState->BufferSize >= FullBufferThreshold)
	{
		FStatsTraceInternal::FlushThreadBuffer(OutThreadState);
	}
	uint64 CycleDiff = Cycle - OutThreadState->LastCycle;
	OutThreadState->LastCycle = Cycle;
	OutBufferPtr = OutThreadState->Buffer + OutThreadState->BufferSize;
	FTraceUtils::Encode7bit((uint64(Stat.GetComparisonIndex().ToUnstableInt()) << 3) | uint64(Op), OutBufferPtr);
	FTraceUtils::Encode7bit(CycleDiff, OutBufferPtr);
}

void FStatsTraceInternal::EndEncodeOp(FThreadState* ThreadState, uint8* BufferPtr)
{
	ThreadState->BufferSize = uint16(BufferPtr - ThreadState->Buffer);
}

void FStatsTrace::DeclareStat(const FName& Stat, const ANSICHAR* Name, const TCHAR* Description, const ANSICHAR* Group, bool IsFloatingPoint, bool IsMemory, bool ShouldClearEveryFrame)
{
	if (!Stat.IsNone() && UE_TRACE_CHANNELEXPR_IS_ENABLED(StatsChannel))
	{
		uint32 NameLen = FCStringAnsi::Strlen(Name);
		uint32 DescriptionLen = FCString::Strlen(Description);
		uint32 GroupLen = FCStringAnsi::Strlen(Group);

		UE_TRACE_LOG(Stats, Spec, StatsChannel, NameLen + DescriptionLen + GroupLen)
			<< Spec.Id(Stat.GetComparisonIndex().ToUnstableInt())
			<< Spec.IsFloatingPoint(IsFloatingPoint)
			<< Spec.IsMemory(IsMemory)
			<< Spec.ShouldClearEveryFrame(ShouldClearEveryFrame)
			<< Spec.Name(Name, NameLen)
			<< Spec.Description(Description, DescriptionLen)
			<< Spec.Group(Group, GroupLen);
	}
}

void FStatsTrace::Increment(const FName& Stat)
{
	if (!Stat.IsNone() && UE_TRACE_CHANNELEXPR_IS_ENABLED(StatsChannel))
	{
		FStatsTraceInternal::FThreadState* ThreadState;
		uint8* BufferPtr;
		FStatsTraceInternal::BeginEncodeOp(Stat, FStatsTraceInternal::Increment, ThreadState, BufferPtr);
		FStatsTraceInternal::EndEncodeOp(ThreadState, BufferPtr);
	}
}

void FStatsTrace::Decrement(const FName& Stat)
{
	if (!Stat.IsNone() && UE_TRACE_CHANNELEXPR_IS_ENABLED(StatsChannel))
	{
		FStatsTraceInternal::FThreadState* ThreadState;
		uint8* BufferPtr;
		FStatsTraceInternal::BeginEncodeOp(Stat, FStatsTraceInternal::Decrement, ThreadState, BufferPtr);
		FStatsTraceInternal::EndEncodeOp(ThreadState, BufferPtr);
	}
}

void FStatsTrace::Add(const FName& Stat, int64 Amount)
{
	if (!Stat.IsNone() && UE_TRACE_CHANNELEXPR_IS_ENABLED(StatsChannel))
	{
		FStatsTraceInternal::FThreadState* ThreadState;
		uint8* BufferPtr;
		FStatsTraceInternal::BeginEncodeOp(Stat, FStatsTraceInternal::AddInteger, ThreadState, BufferPtr);
		FTraceUtils::EncodeZigZag(Amount, BufferPtr);
		FStatsTraceInternal::EndEncodeOp(ThreadState, BufferPtr);
	}
}

void FStatsTrace::Add(const FName& Stat, double Amount)
{
	if (!Stat.IsNone() && UE_TRACE_CHANNELEXPR_IS_ENABLED(StatsChannel))
	{
		FStatsTraceInternal::FThreadState* ThreadState;
		uint8* BufferPtr;
		FStatsTraceInternal::BeginEncodeOp(Stat, FStatsTraceInternal::AddFloat, ThreadState, BufferPtr);
		memcpy(BufferPtr, &Amount, sizeof(double));
		BufferPtr += sizeof(double);
		FStatsTraceInternal::EndEncodeOp(ThreadState, BufferPtr);
	}
}

void FStatsTrace::Set(const FName& Stat, int64 Value)
{
	if (!Stat.IsNone() && UE_TRACE_CHANNELEXPR_IS_ENABLED(StatsChannel))
	{
		FStatsTraceInternal::FThreadState* ThreadState;
		uint8* BufferPtr;
		FStatsTraceInternal::BeginEncodeOp(Stat, FStatsTraceInternal::SetInteger, ThreadState, BufferPtr);
		FTraceUtils::EncodeZigZag(Value, BufferPtr);
		FStatsTraceInternal::EndEncodeOp(ThreadState, BufferPtr);
	}
}

void FStatsTrace::Set(const FName& Stat, double Value)
{
	if (!Stat.IsNone() && UE_TRACE_CHANNELEXPR_IS_ENABLED(StatsChannel))
	{
		FStatsTraceInternal::FThreadState* ThreadState;
		uint8* BufferPtr;
		FStatsTraceInternal::BeginEncodeOp(Stat, FStatsTraceInternal::SetFloat, ThreadState, BufferPtr);
		memcpy(BufferPtr, &Value, sizeof(double));
		BufferPtr += sizeof(double);
		FStatsTraceInternal::EndEncodeOp(ThreadState, BufferPtr);
	}
}

#endif // STATSTRACE_ENABLED
