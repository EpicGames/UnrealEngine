// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stats/ThermalStats.h"
#include "HAL/PlatformMisc.h"

DEFINE_STAT(STAT_DeviceTemperature);
DEFINE_STAT(STAT_ThermalState);

CSV_DEFINE_CATEGORY(Thermal, true);
CSV_DEFINE_STAT(Thermal, DeviceTemperature);
CSV_DEFINE_STAT(Thermal, ThermalState);

namespace UE
{
	namespace ThermalStats
	{
		void GetAndPushStats()
		{
			const float DeviceTemperature = FPlatformMisc::GetDeviceTemperature();
			const EDeviceThermalState ThermalState = FPlatformMisc::GetDeviceThermalState();

			//UE_LOGF(LogStats, Warning, "TemperatureLevel(legacy)=%3.1f DeviceTemperature=%3.1f ThermalState=%i", FPlatformMisc::GetDeviceTemperatureLevel(), DeviceTemperature, ThermalState);

#if CSV_PROFILER
			CSV_CUSTOM_STAT_DEFINED(DeviceTemperature, DeviceTemperature, ECsvCustomStatOp::Set);
			CSV_CUSTOM_STAT_DEFINED(ThermalState, static_cast<int32>(ThermalState), ECsvCustomStatOp::Set);
#endif
#if STATS
			static const FName TemperatureStatName = GET_STATFNAME(STAT_DeviceTemperature);
			static const FName ThermalStateStatName = GET_STATFNAME(STAT_ThermalState);
			SET_FLOAT_STAT_FName(TemperatureStatName, DeviceTemperature);
			SET_DWORD_STAT_FName(ThermalStateStatName, ThermalState);
#endif
		}
	}
}