// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stats/ThreadIdleStats.h"

#if !(defined(DISABLE_THREAD_IDLE_STATS) && DISABLE_THREAD_IDLE_STATS)
#if CPUPROFILERTRACE_ENABLED
UE_TRACE_CHANNEL_DEFINE(ThreadIdleScopeChannel, "Thread idle time tracking. Marks periods when threads are idle or waiting by adding cpu profiler events for them.");
TRACE_CPUPROFILER_EVENT_DECLARE(ThreadIdleScopeTraceEventId);
#endif

UE_DEFINE_THREAD_SINGLETON_TLS(UE::Stats::FThreadIdleStats, CORE_API)

namespace UE::Stats
{

	CORE_API FThreadIdleStats::FScopeIdle::FScopeIdle(bool bInIgnore /* = false */)
		: Start(FPlatformTime::Cycles())
		, bIgnore(bInIgnore || FThreadIdleStats::Get().bInIdleScope)
#if CPUPROFILERTRACE_ENABLED
		, TraceEventScope(ThreadIdleScopeTraceEventId, TEXT("FThreadIdleStats::FScopeIdle"), ThreadIdleScopeChannel, !bIgnore, __FILE__, __LINE__)
#endif
	{
		if (!bIgnore)
		{
			FThreadIdleStats& IdleStats = FThreadIdleStats::Get();
			IdleStats.bInIdleScope = true;
		}
	}
} // namespace UE::Stats

#endif // #if !(defined(DISABLE_THREAD_IDLE_STATS) && DISABLE_THREAD_IDLE_STATS)