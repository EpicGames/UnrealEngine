// Copyright Epic Games, Inc. All Rights Reserved.

#include "String/BytesToHex.h"

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/UnrealString.h"
#include "Misc/StringBuilder.h"
#include "Templates/UnrealTemplate.h"

namespace UE::String
{

template <typename CharType, ANSICHAR LetterA>
static inline void BytesToHexImpl(TConstArrayView<uint8> Bytes, CharType* OutHex)
{
	const auto NibbleToHex = [](uint8 Value) -> CharType { return CharType(Value + (Value > 9 ? LetterA - 10 : '0')); };
	const uint8* Data = Bytes.GetData();
	
	for (const uint8* DataEnd = Data + Bytes.Num(); Data != DataEnd; ++Data)
	{
		*OutHex++ = NibbleToHex(*Data >> 4);
		*OutHex++ = NibbleToHex(*Data & 15);
	}
}

void BytesToHex(TConstArrayView<uint8> Bytes, ANSICHAR* OutHex)
{
	BytesToHexImpl<ANSICHAR, 'A'>(Bytes, OutHex);
}

void BytesToHex(TConstArrayView<uint8> Bytes, WIDECHAR* OutHex)
{
	BytesToHexImpl<WIDECHAR, 'A'>(Bytes, OutHex);
}

void BytesToHex(TConstArrayView<uint8> Bytes, UTF8CHAR* OutHex)
{
	BytesToHexImpl<UTF8CHAR, 'A'>(Bytes, OutHex);
}

void BytesToHexLower(TConstArrayView<uint8> Bytes, ANSICHAR* OutHex)
{
	BytesToHexImpl<ANSICHAR, 'a'>(Bytes, OutHex);
}

void BytesToHexLower(TConstArrayView<uint8> Bytes, WIDECHAR* OutHex)
{
	BytesToHexImpl<WIDECHAR, 'a'>(Bytes, OutHex);
}

void BytesToHexLower(TConstArrayView<uint8> Bytes, UTF8CHAR* OutHex)
{
	BytesToHexImpl<UTF8CHAR, 'a'>(Bytes, OutHex);
}

void BytesToHex(TConstArrayView<uint8> Bytes, FAnsiStringBuilderBase& Builder)
{
	const int32 Offset = Builder.AddUninitialized(Bytes.Num() * 2);
	BytesToHexImpl<ANSICHAR, 'A'>(Bytes, GetData(Builder) + Offset);
}

void BytesToHex(TConstArrayView<uint8> Bytes, FWideStringBuilderBase& Builder)
{
	const int32 Offset = Builder.AddUninitialized(Bytes.Num() * 2);
	BytesToHexImpl<WIDECHAR, 'A'>(Bytes, GetData(Builder) + Offset);
}

void BytesToHex(TConstArrayView<uint8> Bytes, FUtf8StringBuilderBase& Builder)
{
	const int32 Offset = Builder.AddUninitialized(Bytes.Num() * 2);
	BytesToHexImpl<UTF8CHAR, 'A'>(Bytes, GetData(Builder) + Offset);
}

void BytesToHexLower(TConstArrayView<uint8> Bytes, FAnsiStringBuilderBase& Builder)
{
	const int32 Offset = Builder.AddUninitialized(Bytes.Num() * 2);
	BytesToHexImpl<ANSICHAR, 'a'>(Bytes, GetData(Builder) + Offset);
}

void BytesToHexLower(TConstArrayView<uint8> Bytes, FWideStringBuilderBase& Builder)
{
	const int32 Offset = Builder.AddUninitialized(Bytes.Num() * 2);
	BytesToHexImpl<WIDECHAR, 'a'>(Bytes, GetData(Builder) + Offset);
}

void BytesToHexLower(TConstArrayView<uint8> Bytes, FUtf8StringBuilderBase& Builder)
{
	const int32 Offset = Builder.AddUninitialized(Bytes.Num() * 2);
	BytesToHexImpl<UTF8CHAR, 'a'>(Bytes, GetData(Builder) + Offset);
}

} // UE::String

//////////////////// Functions declared in UnrealString.h ////////////////////

void BytesToHex(const uint8* In, int32 Count, FString& Out)
{
	if (Count)
	{
		TArray<TCHAR>& OutArray = Out.GetCharArray();
		OutArray.AddUninitialized(2 * Count + /* add null terminator */ (OutArray.Num() == 0));
		UE::String::BytesToHex(MakeArrayView(In, Count), &OutArray.Last(2 * Count));
		OutArray.Last() = '\0';
	}
}

void BytesToHexLower(const uint8* In, int32 Count, FString& Out)
{
	if (Count)
	{
		TArray<TCHAR>& OutArray = Out.GetCharArray();
		OutArray.AddUninitialized(2 * Count + /* add null terminator */ (OutArray.Num() == 0));
		UE::String::BytesToHexLower(MakeArrayView(In, Count), &OutArray.Last(2 * Count));
		OutArray.Last() = '\0';
	}
}
