// Copyright Epic Games, Inc. All Rights Reserved.

#include "String/CaseConversion.h"

#include "Containers/UnrealString.h"
#include "Internationalization/BreakIterator.h"
#include "Misc/AsciiSet.h"
#include "Misc/StringBuilder.h"

namespace UE::String::Private
{

template <typename CharType>
static inline void UpperCaseTo(TStringView<CharType> Input, TStringBuilderBase<CharType>& Output)
{
	const int32 Offset = Output.AddUninitialized(Input.Len());
	CharType* OutputIterator = Output.GetData() + Offset;
	for (CharType Char : Input)
	{
		*OutputIterator++ = TChar<CharType>::ToUpper(Char);
	}
}

template <typename CharType>
static inline void LowerCaseTo(TStringView<CharType> Input, TStringBuilderBase<CharType>& Output)
{
	const int32 Offset = Output.AddUninitialized(Input.Len());
	CharType* OutputIterator = Output.GetData() + Offset;
	for (CharType Char : Input)
	{
		*OutputIterator++ = TChar<CharType>::ToLower(Char);
	}
}

// @note: Currently only supports TCHAR (as does IBreakIterator)
static inline void PascalCaseTo(FStringView Input, FStringBuilderBase& Output)
{
	// Create a sanitized string, with certain chars removed
	TStringBuilderWithBuffer<TCHAR, 64> CleanStringBuilder;
	CleanStringBuilder.AddUninitialized(Input.Len());

	// Remove apostrophes before converting case, to avoid "You're" becoming "YouRe"
	{
		TCHAR* CleanStringIterator = CleanStringBuilder.GetData();

		int32 NumValidChars = 0;
		constexpr FAsciiSet CharsToRemove("\'\"");
		for (const TCHAR& Chr : Input)
		{
			if (!CharsToRemove.Contains(Chr))
			{
				*CleanStringIterator++ = Chr;
				++NumValidChars;
			}
		}

		const int32 NumCharsToRemove = Input.Len() - NumValidChars;
		CleanStringBuilder.RemoveSuffix(NumCharsToRemove); // Trim excess chars
	}

	const TSharedRef<IBreakIterator> BreakIterator = FBreakIterator::CreateCamelCaseBreakIterator();

	const FStringView CleanStringView(CleanStringBuilder);
	BreakIterator->SetStringRef(CleanStringView);

	Output.Reserve(Output.Len() + CleanStringView.Len());

	{
		// Remove spaces, snake_case, dashes, dots
		constexpr FAsciiSet CharsToRemove(" \t_-.");
		for (int32 PrevBreak = 0, NameBreak = BreakIterator->MoveToNext(); NameBreak != INDEX_NONE; NameBreak = BreakIterator->MoveToNext())
		{
			const TCHAR& Char = CleanStringView[PrevBreak];
			PrevBreak++;

			// Char was a space, etc. Skip over.
			if (CharsToRemove.Contains(Char))
			{
				continue;
			}

			// Uppercase leading character
			Output.AppendChar(TChar<TCHAR>::ToUpper(Char));

			if (PrevBreak < CleanStringBuilder.Len())
			{
				FStringView Trimmed = MakeStringView(&CleanStringView[PrevBreak], NameBreak - PrevBreak);
				Trimmed = FAsciiSet::TrimPrefixWith(Trimmed, CharsToRemove);
				Trimmed = FAsciiSet::TrimSuffixWith(Trimmed, CharsToRemove);

				LowerCaseTo(Trimmed, Output);
			}

			PrevBreak = NameBreak;
		}
	}
}

} // UE::String::Private

namespace UE::String
{

void UpperCaseTo(FAnsiStringView Input, FAnsiStringBuilderBase& Output)
{
	Private::UpperCaseTo(Input, Output);
}

void UpperCaseTo(FUtf8StringView Input, FUtf8StringBuilderBase& Output)
{
	Private::UpperCaseTo(Input, Output);
}

void UpperCaseTo(FWideStringView Input, FWideStringBuilderBase& Output)
{
	Private::UpperCaseTo(Input, Output);
}

void LowerCaseTo(FAnsiStringView Input, FAnsiStringBuilderBase& Output)
{
	Private::LowerCaseTo(Input, Output);
}

void LowerCaseTo(FUtf8StringView Input, FUtf8StringBuilderBase& Output)
{
	Private::LowerCaseTo(Input, Output);
}

void LowerCaseTo(FWideStringView Input, FWideStringBuilderBase& Output)
{
	Private::LowerCaseTo(Input, Output);
}

void PascalCaseTo(FStringView Input, FStringBuilderBase& Output)
{
	Private::PascalCaseTo(Input, Output);
}

} // UE::String
