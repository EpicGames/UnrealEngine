// Copyright Epic Games, Inc. All Rights Reserved.

#include "String/Escape.h"

#include "CoreTypes.h"
#include "Misc/StringBuilder.h"
#include "String/BytesToHex.h"

namespace UE::String
{

void EscapeTo(FStringView Input, FStringBuilderBase& Output)
{
	for (const TCHAR Char : Input)
	{
		if (Char < 0x20)
		{
			switch (Char)
			{
			case '\a': Output.Append(TEXT("\\a")); break;
			case '\b': Output.Append(TEXT("\\b")); break;
			case '\f': Output.Append(TEXT("\\f")); break;
			case '\n': Output.Append(TEXT("\\n")); break;
			case '\r': Output.Append(TEXT("\\r")); break;
			case '\t': Output.Append(TEXT("\\t")); break;
			case '\v': Output.Append(TEXT("\\v")); break;
			default:
				Output.Append(TEXT("\\x"));
				BytesToHexLower({uint8(Char)}, Output);
				break;
			}
		}
		else if (Char < 0x7f)
		{
			switch (Char)
			{
			case '\"': Output.Append(TEXT("\\\"")); break;
			case '\'': Output.Append(TEXT("\\\'")); break;
			case '\\': Output.Append(TEXT("\\\\")); break;
			default:
				Output.AppendChar(Char);
				break;
			}
		}
		else if (Char < 0x80)
		{
			Output.Append(TEXT("\\x"));
			BytesToHexLower({uint8(Char)}, Output);
		}
		else
		{
			Output.Append(TEXT("\\u"));
			BytesToHexLower({uint8(Char >> 8), uint8(Char)}, Output);
		}
	}
}

void QuoteEscapeTo(FStringView Input, FStringBuilderBase& Output)
{
	Output.AppendChar(TEXT('\"'));
	EscapeTo(Input, Output);
	Output.AppendChar(TEXT('\"'));
}

} // UE::String
