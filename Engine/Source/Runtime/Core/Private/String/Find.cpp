// Copyright Epic Games, Inc. All Rights Reserved.

#include "String/Find.h"

#include "Algo/AllOf.h"
#include "Algo/Find.h"
#include "Algo/NoneOf.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/StringView.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Char.h"

namespace UE::String::Private
{

// These are naive implementations that take time proportional to View.Len() * TotalSearchLen.
// If these functions become a bottleneck, they can be specialized separately for one and many search patterns.
// There are algorithms for each that are linear or sub-linear in the length of the string to search.

template <typename CharType>
static inline int32 FindFirst(TStringView<CharType> View, TStringView<CharType> Search, ESearchCase::Type SearchCase)
{
	if (Search.IsEmpty())
	{
		return 0;
	}
	const int32 SearchLen = Search.Len();
	if (SearchLen == 1)
	{
		return String::FindFirstChar(View, Search[0], SearchCase);
	}
	const CharType* const SearchData = Search.GetData();
	const CharType* const ViewBegin = View.GetData();
	const int32 ViewLen = View.Len();
	if (ViewLen < SearchLen)
	{
		return INDEX_NONE;
	}
	const CharType* const ViewEnd = ViewBegin + ViewLen - SearchLen;
	if (SearchCase == ESearchCase::CaseSensitive)
	{
		for (const CharType* ViewIt = ViewBegin; ViewIt <= ViewEnd; ++ViewIt)
		{
			if (TCString<CharType>::Strncmp(ViewIt, SearchData, SearchLen) == 0)
			{
				return int32(ViewIt - ViewBegin);
			}
		}
	}
	else
	{
		for (const CharType* ViewIt = ViewBegin; ViewIt <= ViewEnd; ++ViewIt)
		{
			if (TCString<CharType>::Strnicmp(ViewIt, SearchData, SearchLen) == 0)
			{
				return int32(ViewIt - ViewBegin);
			}
		}
	}
	return INDEX_NONE;
}

template <typename CharType>
static inline int32 FindLast(TStringView<CharType> View, TStringView<CharType> Search, ESearchCase::Type SearchCase)
{
	if (Search.IsEmpty())
	{
		return View.Len();
	}
	const int32 SearchLen = Search.Len();
	if (SearchLen == 1)
	{
		return String::FindLastChar(View, Search[0], SearchCase);
	}
	const CharType* const SearchData = Search.GetData();
	const CharType* const ViewBegin = View.GetData();
	const int32 ViewLen = View.Len();
	if (ViewLen < SearchLen)
	{
		return INDEX_NONE;
	}
	const CharType* const ViewEnd = ViewBegin + ViewLen - SearchLen;
	if (SearchCase == ESearchCase::CaseSensitive)
	{
		for (const CharType* ViewIt = ViewEnd; ViewIt >= ViewBegin; --ViewIt)
		{
			if (TCString<CharType>::Strncmp(ViewIt, SearchData, SearchLen) == 0)
			{
				return int32(ViewIt - ViewBegin);
			}
		}
	}
	else
	{
		for (const CharType* ViewIt = ViewEnd; ViewIt >= ViewBegin; --ViewIt)
		{
			if (TCString<CharType>::Strnicmp(ViewIt, SearchData, SearchLen) == 0)
			{
				return int32(ViewIt - ViewBegin);
			}
		}
	}
	return INDEX_NONE;
}

template <typename CharType>
static inline int32 FindFirstOfAny(TStringView<CharType> View, TConstArrayView<TStringView<CharType>> Search, ESearchCase::Type SearchCase, int32* OutMatchIndex)
{
	switch (Search.Num())
	{
	case 0:
		return INDEX_NONE;
	case 1:
	{
		const int32 Index = String::FindFirst(View, Search[0], SearchCase);
		if (Index != INDEX_NONE && OutMatchIndex)
		{
			*OutMatchIndex = 0;
		}
		return Index;
	}
	default:
		if (Algo::AllOf(Search, [](const TStringView<CharType>& Pattern) { return Pattern.Len() == 1; }))
		{
			TArray<CharType, TInlineAllocator<32>> SearchChars;
			SearchChars.Reserve(Search.Num());
			for (const TStringView<CharType>& Pattern : Search)
			{
				SearchChars.Add(Pattern[0]);
			}
			return String::FindFirstOfAnyChar(View, SearchChars, SearchCase, OutMatchIndex);
		}
		break;
	}

	const CharType* const ViewBegin = View.GetData();
	const int32 ViewLen = View.Len();
	if (ViewLen > 0)
	{
		for (int32 ViewIndex = 0; ViewIndex != ViewLen; ++ViewIndex)
		{
			const TStringView<CharType> RemainingView(ViewBegin + ViewIndex, ViewLen - ViewIndex);
			auto MatchPattern = [&RemainingView, SearchCase](const TStringView<CharType>& Pattern) { return RemainingView.StartsWith(Pattern, SearchCase); };
			if (const TStringView<CharType>* Match = Algo::FindByPredicate(Search, MatchPattern))
			{
				if (OutMatchIndex)
				{
					*OutMatchIndex = UE_PTRDIFF_TO_INT32(Match - Search.GetData());
				}
				return ViewIndex;
			}
		}
	}
	else
	{
		// An empty view can match an empty pattern.
		auto MatchEmptyPattern = [](const TStringView<CharType>& Pattern) { return Pattern.IsEmpty(); };
		if (const TStringView<CharType>* Match = Algo::FindByPredicate(Search, MatchEmptyPattern))
		{
			if (OutMatchIndex)
			{
				*OutMatchIndex = UE_PTRDIFF_TO_INT32(Match - Search.GetData());;
			}
			return 0;
		}
	}
	return INDEX_NONE;
}

template <typename CharType>
static inline int32 FindLastOfAny(TStringView<CharType> View, TConstArrayView<TStringView<CharType>> Search, ESearchCase::Type SearchCase, int32* OutMatchIndex)
{
	switch (Search.Num())
	{
	case 0:
		return INDEX_NONE;
	case 1:
	{
		const int32 Index = String::FindLast(View, Search[0], SearchCase);
		if (Index != INDEX_NONE && OutMatchIndex)
		{
			*OutMatchIndex = 0;
		}
		return Index;
	}
	default:
		if (Algo::AllOf(Search, [](const TStringView<CharType>& Pattern) { return Pattern.Len() == 1; }))
		{
			TArray<CharType, TInlineAllocator<32>> SearchChars;
			SearchChars.Reserve(Search.Num());
			for (const TStringView<CharType>& Pattern : Search)
			{
				SearchChars.Add(Pattern[0]);
			}
			return String::FindLastOfAnyChar(View, SearchChars, SearchCase, OutMatchIndex);
		}
		break;
	}

	const CharType* const ViewBegin = View.GetData();
	const int32 ViewLen = View.Len();
	if (ViewLen > 0)
	{
		for (int32 ViewIndex = ViewLen; ViewIndex >= 0; --ViewIndex)
		{
			const TStringView<CharType> RemainingView(ViewBegin + ViewIndex, ViewLen - ViewIndex);
			auto MatchPattern = [&RemainingView, SearchCase](const TStringView<CharType>& Pattern) { return RemainingView.StartsWith(Pattern, SearchCase); };
			if (const TStringView<CharType>* Match = Algo::FindByPredicate(Search, MatchPattern))
			{
				if (OutMatchIndex)
				{
					*OutMatchIndex = UE_PTRDIFF_TO_INT32(Match - Search.GetData());
				}
				return ViewIndex;
			}
		}
	}
	else
	{
		// An empty view can match an empty pattern.
		auto MatchEmptyPattern = [](const TStringView<CharType>& Pattern) { return Pattern.IsEmpty(); };
		if (const TStringView<CharType>* Match = Algo::FindByPredicate(Search, MatchEmptyPattern))
		{
			if (OutMatchIndex)
			{
				*OutMatchIndex = UE_PTRDIFF_TO_INT32(Match - Search.GetData());;
			}
			return 0;
		}
	}
	return INDEX_NONE;
}

template <typename CharType>
static inline int32 FindFirstChar(TStringView<CharType> View, CharType Search, ESearchCase::Type SearchCase)
{
	const CharType* const ViewBegin = View.GetData();
	const CharType* const ViewEnd = ViewBegin + View.Len();
	if (SearchCase == ESearchCase::CaseSensitive)
	{
		for (const CharType* ViewIt = ViewBegin; ViewIt < ViewEnd; ++ViewIt)
		{
			if (*ViewIt == Search)
			{
				return int32(ViewIt - ViewBegin);
			}
		}
	}
	else
	{
		const CharType SearchUpper = TChar<CharType>::ToUpper(Search);
		for (const CharType* ViewIt = ViewBegin; ViewIt < ViewEnd; ++ViewIt)
		{
			if (TChar<CharType>::ToUpper(*ViewIt) == SearchUpper)
			{
				return int32(ViewIt - ViewBegin);
			}
		}
	}
	return INDEX_NONE;
}

template <typename CharType>
static inline int32 FindLastChar(TStringView<CharType> View, CharType Search, ESearchCase::Type SearchCase)
{
	const CharType* const ViewBegin = View.GetData();
	const CharType* const ViewEnd = ViewBegin + View.Len();
	if (ViewEnd == ViewBegin)
	{
		return INDEX_NONE;
	}
	if (SearchCase == ESearchCase::CaseSensitive)
	{
		for (const CharType* ViewIt = ViewEnd - 1; ViewIt >= ViewBegin; --ViewIt)
		{
			if (*ViewIt == Search)
			{
				return int32(ViewIt - ViewBegin);
			}
		}
	}
	else
	{
		const CharType SearchUpper = TChar<CharType>::ToUpper(Search);
		for (const CharType* ViewIt = ViewEnd - 1; ViewIt >= ViewBegin; --ViewIt)
		{
			if (TChar<CharType>::ToUpper(*ViewIt) == SearchUpper)
			{
				return int32(ViewIt - ViewBegin);
			}
		}
	}
	return INDEX_NONE;
}

template <typename CharType>
static inline int32 FindFirstOfAnyChar(TStringView<CharType> View, TConstArrayView<CharType> Search, ESearchCase::Type SearchCase, int32* OutMatchIndex)
{
	switch (Search.Num())
	{
	case 0:
		return INDEX_NONE;
	case 1:
	{
		const int32 Index = String::FindFirstChar(View, Search[0], SearchCase);
		if (OutMatchIndex)
		{
			*OutMatchIndex = 0;
		}
		return Index;
	}
	default:
		break;
	}

	const CharType* const ViewBegin = View.GetData();
	const CharType* const ViewEnd = ViewBegin + View.Len();
	if (SearchCase == ESearchCase::CaseSensitive)
	{
		for (const CharType* ViewIt = ViewBegin; ViewIt < ViewEnd; ++ViewIt)
		{
			if (const CharType* Match = Algo::Find(Search, *ViewIt))
			{
				if (OutMatchIndex)
				{
					*OutMatchIndex = UE_PTRDIFF_TO_INT32(Match - Search.GetData());
				}
				return int32(ViewIt - ViewBegin);
			}
		}
		return INDEX_NONE;
	}
	else
	{
		TArray<CharType, TInlineAllocator<32>> SearchUpper;
		SearchUpper.Reserve(Search.Num());
		for (const CharType Pattern : Search)
		{
			SearchUpper.Add(TChar<CharType>::ToUpper(Pattern));
		}
		for (const CharType* ViewIt = ViewBegin; ViewIt < ViewEnd; ++ViewIt)
		{
			if (const CharType* Match = Algo::Find(SearchUpper, TChar<CharType>::ToUpper(*ViewIt)))
			{
				if (OutMatchIndex)
				{
					*OutMatchIndex = UE_PTRDIFF_TO_INT32(Match - SearchUpper.GetData());
				}
				return int32(ViewIt - ViewBegin);
			}
		}
		return INDEX_NONE;
	}
}

template <typename CharType>
static inline int32 FindLastOfAnyChar(TStringView<CharType> View, TConstArrayView<CharType> Search, ESearchCase::Type SearchCase, int32* OutMatchIndex)
{
	switch (Search.Num())
	{
	case 0:
		return INDEX_NONE;
	case 1:
	{
		const int32 Index = String::FindLastChar(View, Search[0], SearchCase);
		if (OutMatchIndex)
		{
			*OutMatchIndex = 0;
		}
		return Index;
	}
	default:
		break;
	}

	const CharType* const ViewBegin = View.GetData();
	const CharType* const ViewEnd = ViewBegin + View.Len();
	if (ViewEnd == ViewBegin)
	{
		return INDEX_NONE;
	}
	if (SearchCase == ESearchCase::CaseSensitive)
	{
		for (const CharType* ViewIt = ViewEnd - 1; ViewIt >= ViewBegin; --ViewIt)
		{
			if (const CharType* Match = Algo::Find(Search, *ViewIt))
			{
				if (OutMatchIndex)
				{
					*OutMatchIndex = UE_PTRDIFF_TO_INT32(Match - Search.GetData());
				}
				return int32(ViewIt - ViewBegin);
			}
		}
		return INDEX_NONE;
	}
	else
	{
		TArray<CharType, TInlineAllocator<32>> SearchUpper;
		SearchUpper.Reserve(Search.Num());
		for (const CharType Pattern : Search)
		{
			SearchUpper.Add(TChar<CharType>::ToUpper(Pattern));
		}
		for (const CharType* ViewIt = ViewEnd - 1; ViewIt >= ViewBegin; --ViewIt)
		{
			if (const CharType* Match = Algo::Find(SearchUpper, TChar<CharType>::ToUpper(*ViewIt)))
			{
				if (OutMatchIndex)
				{
					*OutMatchIndex = UE_PTRDIFF_TO_INT32(Match - SearchUpper.GetData());
				}
				return int32(ViewIt - ViewBegin);
			}
		}
		return INDEX_NONE;
	}
}

} // UE::String::Private

namespace UE::String
{

int32 FindFirst(FUtf8StringView View, FUtf8StringView Search, ESearchCase::Type SearchCase)
{
	return Private::FindFirst(View, Search, SearchCase);
}

int32 FindFirst(FWideStringView View, FWideStringView Search, ESearchCase::Type SearchCase)
{
	return Private::FindFirst(View, Search, SearchCase);
}

int32 FindLast(FUtf8StringView View, FUtf8StringView Search, ESearchCase::Type SearchCase)
{
	return Private::FindLast(View, Search, SearchCase);
}

int32 FindLast(FWideStringView View, FWideStringView Search, ESearchCase::Type SearchCase)
{
	return Private::FindLast(View, Search, SearchCase);
}

int32 FindFirstOfAny(FAnsiStringView View, TConstArrayView<FAnsiStringView> Search, ESearchCase::Type SearchCase, int32* OutMatchIndex)
{
	return Private::FindFirstOfAny(View, Search, SearchCase, OutMatchIndex);
}

int32 FindFirstOfAny(FUtf8StringView View, TConstArrayView<FUtf8StringView> Search, ESearchCase::Type SearchCase, int32* OutMatchIndex)
{
	return Private::FindFirstOfAny(View, Search, SearchCase, OutMatchIndex);
}

int32 FindFirstOfAny(FWideStringView View, TConstArrayView<FWideStringView> Search, ESearchCase::Type SearchCase, int32* OutMatchIndex)
{
	return Private::FindFirstOfAny(View, Search, SearchCase, OutMatchIndex);
}

int32 FindLastOfAny(FAnsiStringView View, TConstArrayView<FAnsiStringView> Search, ESearchCase::Type SearchCase, int32* OutMatchIndex)
{
	return Private::FindLastOfAny(View, Search, SearchCase, OutMatchIndex);
}

int32 FindLastOfAny(FUtf8StringView View, TConstArrayView<FUtf8StringView> Search, ESearchCase::Type SearchCase, int32* OutMatchIndex)
{
	return Private::FindLastOfAny(View, Search, SearchCase, OutMatchIndex);
}

int32 FindLastOfAny(FWideStringView View, TConstArrayView<FWideStringView> Search, ESearchCase::Type SearchCase, int32* OutMatchIndex)
{
	return Private::FindLastOfAny(View, Search, SearchCase, OutMatchIndex);
}

int32 FindFirstChar(FUtf8StringView View, UTF8CHAR Search, ESearchCase::Type SearchCase)
{
	return Private::FindFirstChar(View, Search, SearchCase);
}

int32 FindFirstChar(FWideStringView View, WIDECHAR Search, ESearchCase::Type SearchCase)
{
	return Private::FindFirstChar(View, Search, SearchCase);
}

int32 FindLastChar(FUtf8StringView View, UTF8CHAR Search, ESearchCase::Type SearchCase)
{
	return Private::FindLastChar(View, Search, SearchCase);
}

int32 FindLastChar(FWideStringView View, WIDECHAR Search, ESearchCase::Type SearchCase)
{
	return Private::FindLastChar(View, Search, SearchCase);
}

int32 FindFirstOfAnyChar(FUtf8StringView View, std::initializer_list<UTF8CHAR> Search, ESearchCase::Type SearchCase, int32* OutMatchIndex)
{
	return Private::FindFirstOfAnyChar(View, MakeArrayView(Search), SearchCase, OutMatchIndex);
}

int32 FindFirstOfAnyChar(FUtf8StringView View, TConstArrayView<ANSICHAR> Search, ESearchCase::Type SearchCase, int32* OutMatchIndex)
{
	return Private::FindFirstOfAnyChar(View, MakeArrayView((const UTF8CHAR*)Search.GetData(), Search.Num()), SearchCase, OutMatchIndex);
}

int32 FindFirstOfAnyChar(FUtf8StringView View, TConstArrayView<UTF8CHAR> Search, ESearchCase::Type SearchCase, int32* OutMatchIndex)
{
	return Private::FindFirstOfAnyChar(View, Search, SearchCase, OutMatchIndex);
}

int32 FindFirstOfAnyChar(FWideStringView View, TConstArrayView<WIDECHAR> Search, ESearchCase::Type SearchCase, int32* OutMatchIndex)
{
	return Private::FindFirstOfAnyChar(View, Search, SearchCase, OutMatchIndex);
}

int32 FindLastOfAnyChar(FUtf8StringView View, std::initializer_list<UTF8CHAR> Search, ESearchCase::Type SearchCase, int32* OutMatchIndex)
{
	return Private::FindLastOfAnyChar(View, MakeArrayView(Search), SearchCase, OutMatchIndex);
}

int32 FindLastOfAnyChar(FUtf8StringView View, TConstArrayView<ANSICHAR> Search, ESearchCase::Type SearchCase, int32* OutMatchIndex)
{
	return Private::FindLastOfAnyChar(View, MakeArrayView((const UTF8CHAR*)Search.GetData(), Search.Num()), SearchCase, OutMatchIndex);
}

int32 FindLastOfAnyChar(FUtf8StringView View, TConstArrayView<UTF8CHAR> Search, ESearchCase::Type SearchCase, int32* OutMatchIndex)
{
	return Private::FindLastOfAnyChar(View, Search, SearchCase, OutMatchIndex);
}

int32 FindLastOfAnyChar(FWideStringView View, TConstArrayView<WIDECHAR> Search, ESearchCase::Type SearchCase, int32* OutMatchIndex)
{
	return Private::FindLastOfAnyChar(View, Search, SearchCase, OutMatchIndex);
}

} // UE::String
