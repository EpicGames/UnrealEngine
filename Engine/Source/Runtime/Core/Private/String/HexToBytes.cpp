// Copyright Epic Games, Inc. All Rights Reserved.

#include "String/HexToBytes.h"

#include "Containers/StringView.h"
#include "Containers/UnrealString.h"

namespace UE::String
{

template <typename CharType>
static inline int32 HexToBytesImpl(TStringView<CharType> Hex, uint8* const OutBytes)
{
	const int32 HexCount = Hex.Len();
	const CharType* HexPos = Hex.GetData();
	const CharType* HexEnd = HexPos + HexCount;
	uint8* OutPos = OutBytes;
	if (const bool bPadNibble = (HexCount % 2) == 1)
	{
		*OutPos++ = TCharToNibble(static_cast<TCHAR>(*HexPos++));
	}
	while (HexPos != HexEnd)
	{
		const uint8 HiNibble = uint8(TCharToNibble(static_cast<TCHAR>(*HexPos++)) << 4);
		*OutPos++ = HiNibble | TCharToNibble(static_cast<TCHAR>(*HexPos++));
	}
	return static_cast<int32>(OutPos - OutBytes);
}

int32 HexToBytes(FWideStringView Hex, uint8* OutBytes)
{
	return HexToBytesImpl<WIDECHAR>(Hex, OutBytes);
}

int32 HexToBytes(FUtf8StringView Hex, uint8* OutBytes)
{
	return HexToBytesImpl<UTF8CHAR>(Hex, OutBytes);
}

} // UE::String
