// Copyright Epic Games, Inc. All Rights Reserved.

#include "String/LexFromString.h"

#include "Containers/UnrealString.h"
#include "Containers/Utf8String.h"
#include "Misc/StringBuilder.h"

void LexFromString(int8& OutValue, const FStringView& InString)
{
	TStringBuilder<64> Builder;
	LexFromString(OutValue, *(Builder << InString));
}

void LexFromString(int16& OutValue, const FStringView& InString)
{
	TStringBuilder<64> Builder;
	LexFromString(OutValue, *(Builder << InString));
}

void LexFromString(int32& OutValue, const FStringView& InString)
{
	TStringBuilder<64> Builder;
	LexFromString(OutValue, *(Builder << InString));
}

void LexFromString(int64& OutValue, const FStringView& InString)
{
	TStringBuilder<64> Builder;
	LexFromString(OutValue, *(Builder << InString));
}

void LexFromString(uint8& OutValue, const FStringView& InString)
{
	TStringBuilder<64> Builder;
	LexFromString(OutValue, *(Builder << InString));
}

void LexFromString(uint16& OutValue, const FStringView& InString)
{
	TStringBuilder<64> Builder;
	LexFromString(OutValue, *(Builder << InString));
}

void LexFromString(uint32& OutValue, const FStringView& InString)
{
	TStringBuilder<64> Builder;
	LexFromString(OutValue, *(Builder << InString));
}

void LexFromString(uint64& OutValue, const FStringView& InString)
{
	TStringBuilder<64> Builder;
	LexFromString(OutValue, *(Builder << InString));
}

void LexFromString(float& OutValue, const FStringView& InString)
{
	TStringBuilder<64> Builder;
	LexFromString(OutValue, *(Builder << InString));
}

void LexFromString(double& OutValue, const FStringView& InString)
{
	TStringBuilder<64> Builder;
	LexFromString(OutValue, *(Builder << InString));
}

void LexFromString(bool& OutValue, const FStringView& InString)
{
	TStringBuilder<64> Builder;
	LexFromString(OutValue, *(Builder << InString));
}

void LexFromString(int8& OutValue, const FUtf8StringView& InString)
{
	TUtf8StringBuilder<64> Builder;
	LexFromString(OutValue, *(Builder << InString));
}

void LexFromString(int16& OutValue, const FUtf8StringView& InString)
{
	TUtf8StringBuilder<64> Builder;
	LexFromString(OutValue, *(Builder << InString));
}

void LexFromString(int32& OutValue, const FUtf8StringView& InString)
{
	TUtf8StringBuilder<64> Builder;
	LexFromString(OutValue, *(Builder << InString));
}

void LexFromString(int64& OutValue, const FUtf8StringView& InString)
{
	TUtf8StringBuilder<64> Builder;
	LexFromString(OutValue, *(Builder << InString));
}

void LexFromString(uint8& OutValue, const FUtf8StringView& InString)
{
	TUtf8StringBuilder<64> Builder;
	LexFromString(OutValue, *(Builder << InString));
}

void LexFromString(uint16& OutValue, const FUtf8StringView& InString)
{
	TUtf8StringBuilder<64> Builder;
	LexFromString(OutValue, *(Builder << InString));
}

void LexFromString(uint32& OutValue, const FUtf8StringView& InString)
{
	TUtf8StringBuilder<64> Builder;
	LexFromString(OutValue, *(Builder << InString));
}

void LexFromString(uint64& OutValue, const FUtf8StringView& InString)
{
	TUtf8StringBuilder<64> Builder;
	LexFromString(OutValue, *(Builder << InString));
}

void LexFromString(float& OutValue, const FUtf8StringView& InString)
{
	TUtf8StringBuilder<64> Builder;
	LexFromString(OutValue, *(Builder << InString));
}

void LexFromString(double& OutValue, const FUtf8StringView& InString)
{
	TUtf8StringBuilder<64> Builder;
	LexFromString(OutValue, *(Builder << InString));
}

void LexFromString(bool& OutValue, const FUtf8StringView& InString)
{
	TUtf8StringBuilder<64> Builder;
	LexFromString(OutValue, *(Builder << InString));
}
