// Copyright Epic Games, Inc. All Rights Reserved.

#include "String/LineEndings.h"

#include "CoreTypes.h"
#include "Misc/CString.h"
#include "Templates/UnrealTemplate.h"

namespace UE::String
{
	CORE_API FString FromHostLineEndings(const FString& InString)
	{
		FString New = InString;
		FromHostLineEndingsInline(New);
		return New;
	}

	CORE_API FString FromHostLineEndings(FString&& InString)
	{
		FromHostLineEndingsInline(InString);
		return MoveTemp(InString);
	}

	CORE_API void FromHostLineEndingsInline(FString& InString)
	{
		InString.ReplaceInline(TEXT("\r\n"), TEXT("\n"));
		InString.ReplaceInline(TEXT("\r"), TEXT("\n"));
	}

	CORE_API FString ToHostLineEndings(const FString& InString)
	{
		FString New = InString;
		ToHostLineEndingsInline(New);
		return New;
	}

	CORE_API FString ToHostLineEndings(FString&& InString)
	{
		ToHostLineEndingsInline(InString);
		return MoveTemp(InString);
	}

	CORE_API void ToHostLineEndingsInline(FString& InString)
	{
		FromHostLineEndingsInline(InString);

		if (TCString<TCHAR>::Strcmp(TEXT("\n"), LINE_TERMINATOR) != 0)
		{
			InString.ReplaceInline(TEXT("\n"), LINE_TERMINATOR);
		}
	}
} // UE::String
