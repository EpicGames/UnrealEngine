// Copyright Epic Games, Inc. All Rights Reserved.

#include "String/Numeric.h"
#include "Containers/StringView.h"

namespace UE::String::Private
{

template <typename CharType>
static bool IsNumeric(TStringView<CharType> View)
{
	if (View.IsEmpty())
	{
		return false;
	}

	const CharType* CurPtr = View.GetData();
	const CharType* EndPtr = CurPtr + View.Len();

	if (*CurPtr == '-' || *CurPtr == '+')
	{
		++CurPtr;
	}

	bool bHasDot = false;
	while (CurPtr < EndPtr)
	{
		if (*CurPtr == '.')
		{
			if (bHasDot)
			{
				return false;
			}

			bHasDot = true;
		}
		else if (!TChar<CharType>::IsDigit(*CurPtr))
		{
			return false;
		}

		++CurPtr;
	}

	return true;
}

template <typename CharType>
static bool IsNumericOnlyDigits(TStringView<CharType> View)
{
	if (View.IsEmpty())
	{
		return false;
	}

	const CharType* CurPtr = View.GetData();
	const CharType* EndPtr = CurPtr + View.Len();

	while (CurPtr < EndPtr)
	{
		if (!TChar<CharType>::IsDigit(*CurPtr))
		{
			return false;
		}

		++CurPtr;
	}

	return true;
}

} // UE::String::Private

namespace UE::String
{

bool IsNumeric(FWideStringView View)
{
	return Private::IsNumeric(View);
}

bool IsNumeric(FUtf8StringView View)
{
	return Private::IsNumeric(View);
}

bool IsNumericOnlyDigits(FWideStringView View)
{
	return Private::IsNumericOnlyDigits(View);
}

bool IsNumericOnlyDigits(FUtf8StringView View)
{
	return Private::IsNumericOnlyDigits(View);
}

} // UE::String
