// Copyright Epic Games, Inc. All Rights Reserved.

#include "String/ParseLines.h"

#include "Templates/Function.h"

namespace UE::String
{

template <typename CharType>
inline static void ParseLinesImpl(
	const TStringView<CharType> View,
	const TFunctionRef<void (TStringView<CharType>)> Visitor,
	const EParseLinesOptions Options)
{
	const CharType* ViewIt = View.GetData();
	const CharType* ViewEnd = ViewIt + View.Len();
	do
	{
		const CharType* LineStart = ViewIt;
		const CharType* LineEnd = ViewEnd;
		for (; ViewIt != ViewEnd; ++ViewIt)
		{
			const CharType CurrentChar = *ViewIt;
			if (CurrentChar == CharType('\n'))
			{
				LineEnd = ViewIt++;
				break;
			}
			if (CurrentChar == CharType('\r'))
			{
				LineEnd = ViewIt++;
				if (ViewIt != ViewEnd && *ViewIt == CharType('\n'))
				{
					++ViewIt;
				}
				break;
			}
		}

		TStringView<CharType> Line(LineStart, UE_PTRDIFF_TO_INT32(LineEnd - LineStart));
		if (EnumHasAnyFlags(Options, EParseLinesOptions::Trim))
		{
			Line = Line.TrimStartAndEnd();
		}
		if (!EnumHasAnyFlags(Options, EParseLinesOptions::SkipEmpty) || !Line.IsEmpty())
		{
			Visitor(Line);
		}
	}
	while (ViewIt != ViewEnd);
}

void ParseLines(
	FWideStringView View,
	TFunctionRef<void (FWideStringView)> Visitor,
	EParseLinesOptions Options)
{
	return ParseLinesImpl(View, Visitor, Options);
}

void ParseLines(
	FUtf8StringView View,
	TFunctionRef<void (FUtf8StringView)> Visitor,
	EParseLinesOptions Options)
{
	return ParseLinesImpl(View, Visitor, Options);
}

} // UE::String
