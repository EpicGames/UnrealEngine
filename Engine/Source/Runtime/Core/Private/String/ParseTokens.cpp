// Copyright Epic Games, Inc. All Rights Reserved.

#include "String/ParseTokens.h"

#include "Algo/AllOf.h"
#include "Algo/Find.h"
#include "Algo/NoneOf.h"
#include "Algo/Transform.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/BitArray.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CString.h"
#include "Misc/Char.h"
#include "Templates/Function.h"
#include "Templates/UnrealTemplate.h"

namespace UE::String::Private
{

template <typename CharType>
inline static void ParseTokensVisitToken(
	const TFunctionRef<void (TStringView<CharType>)>& Visitor,
	const EParseTokensOptions Options,
	TStringView<CharType> Token)
{
	if (EnumHasAnyFlags(Options, EParseTokensOptions::Trim))
	{
		Token = Token.TrimStartAndEnd();
	}
	if (!EnumHasAnyFlags(Options, EParseTokensOptions::SkipEmpty) || !Token.IsEmpty())
	{
		Visitor(Token);
	}
}

/** Parse tokens with one single-character delimiter. */
template <typename CharType>
inline static void ParseTokens1Delim1Char(
	const TStringView<CharType> View,
	const CharType Delimiter,
	const TFunctionRef<void (TStringView<CharType>)> Visitor,
	const EParseTokensOptions Options)
{
	const CharType* ViewIt = View.GetData();
	const CharType* const ViewEnd = ViewIt + View.Len();
	const CharType* NextToken = ViewIt;

	if (EnumHasAnyFlags(Options, EParseTokensOptions::IgnoreCase))
	{
		const CharType LowerDelimiter = TChar<CharType>::ToLower(Delimiter);
		for (;;)
		{
			if (ViewIt == ViewEnd)
			{
				break;
			}
			if (TChar<CharType>::ToLower(*ViewIt) != LowerDelimiter)
			{
				++ViewIt;
				continue;
			}
			ParseTokensVisitToken(Visitor, Options, TStringView<CharType>(NextToken, UE_PTRDIFF_TO_INT32(ViewIt - NextToken)));
			NextToken = ++ViewIt;
		}
	}
	else
	{
		for (;;)
		{
			if (ViewIt == ViewEnd)
			{
				break;
			}
			if (*ViewIt != Delimiter)
			{
				++ViewIt;
				continue;
			}
			ParseTokensVisitToken(Visitor, Options, TStringView<CharType>(NextToken, UE_PTRDIFF_TO_INT32(ViewIt - NextToken)));
			NextToken = ++ViewIt;
		}
	}

	ParseTokensVisitToken(Visitor, Options, TStringView<CharType>(NextToken, UE_PTRDIFF_TO_INT32(ViewIt - NextToken)));
}

/** Parse tokens with multiple single-character Basic Latin delimiters. */
template <typename CharType>
inline static void ParseTokensNDelim1CharBasicLatin(
	const TStringView<CharType> View,
	const TConstArrayView<CharType> Delimiters,
	const TFunctionRef<void (TStringView<CharType>)> Visitor,
	const EParseTokensOptions Options)
{
	TBitArray<> DelimiterMask(false, 128);
	if (EnumHasAnyFlags(Options, EParseTokensOptions::IgnoreCase))
	{
		for (CharType Delimiter : Delimiters)
		{
			DelimiterMask[TChar<CharType>::ToUnsigned(TChar<CharType>::ToLower(Delimiter))] = true;
			DelimiterMask[TChar<CharType>::ToUnsigned(TChar<CharType>::ToUpper(Delimiter))] = true;
		}
	}
	else
	{
		for (CharType Delimiter : Delimiters)
		{
			DelimiterMask[TChar<CharType>::ToUnsigned(Delimiter)] = true;
		}
	}

	const CharType* ViewIt = View.GetData();
	const CharType* const ViewEnd = ViewIt + View.Len();
	const CharType* NextToken = ViewIt;

	for (;;)
	{
		if (ViewIt == ViewEnd)
		{
			break;
		}
		const uint32 CodePoint = *ViewIt;
		if (CodePoint >= 128 || !DelimiterMask[CodePoint])
		{
			++ViewIt;
			continue;
		}
		ParseTokensVisitToken(Visitor, Options, TStringView<CharType>(NextToken, UE_PTRDIFF_TO_INT32(ViewIt - NextToken)));
		NextToken = ++ViewIt;
	}

	ParseTokensVisitToken(Visitor, Options, TStringView<CharType>(NextToken, UE_PTRDIFF_TO_INT32(ViewIt - NextToken)));
}

/** Parse tokens with multiple single-character delimiters in the Basic Multilingual Plane. */
template <typename CharType>
inline static void ParseTokensNDelim1Char(
	const TStringView<CharType> View,
	const TConstArrayView<CharType> Delimiters,
	TFunctionRef<void (TStringView<CharType>)> Visitor,
	const EParseTokensOptions Options)
{
	if (Algo::AllOf(Delimiters, [](CharType Delimiter) { return Delimiter < 128; }))
	{
		return ParseTokensNDelim1CharBasicLatin(View, Delimiters, MoveTemp(Visitor), Options);
	}

	const CharType* ViewIt = View.GetData();
	const CharType* const ViewEnd = ViewIt + View.Len();
	const CharType* NextToken = ViewIt;

	if (EnumHasAnyFlags(Options, EParseTokensOptions::IgnoreCase))
	{
		TArray<CharType, TInlineAllocator<16>> LowerDelimiters;
		Algo::Transform(Delimiters, LowerDelimiters, TChar<CharType>::ToLower);
		for (;;)
		{
			if (ViewIt == ViewEnd)
			{
				break;
			}
			if (!Algo::Find(Delimiters, TChar<CharType>::ToLower(*ViewIt)))
			{
				++ViewIt;
				continue;
			}
			ParseTokensVisitToken(Visitor, Options, TStringView<CharType>(NextToken, UE_PTRDIFF_TO_INT32(ViewIt - NextToken)));
			NextToken = ++ViewIt;
		}
	}
	else
	{
		for (;;)
		{
			if (ViewIt == ViewEnd)
			{
				break;
			}
			if (!Algo::Find(Delimiters, *ViewIt))
			{
				++ViewIt;
				continue;
			}
			ParseTokensVisitToken(Visitor, Options, TStringView<CharType>(NextToken, UE_PTRDIFF_TO_INT32(ViewIt - NextToken)));
			NextToken = ++ViewIt;
		}
	}

	ParseTokensVisitToken(Visitor, Options, TStringView<CharType>(NextToken, UE_PTRDIFF_TO_INT32(ViewIt - NextToken)));
}

/** Parse tokens with multiple multi-character delimiters. */
template <typename CharType>
inline static void ParseTokensNDelimNChar(
	const TStringView<CharType> View,
	const TConstArrayView<TStringView<CharType>> Delimiters,
	const TFunctionRef<void (TStringView<CharType>)> Visitor,
	const EParseTokensOptions Options)
{
	// This is a naive implementation that takes time proportional to View.Len() * TotalDelimiterLen.
	// If this function becomes a bottleneck, it can be specialized separately for one and many delimiters.
	// There are algorithms for each are linear or sub-linear in the length of string to search.

	const int32 ViewLen = View.Len();
	int32 NextTokenIndex = 0;

	const ESearchCase::Type SearchCase = EnumHasAnyFlags(Options, EParseTokensOptions::IgnoreCase) ? ESearchCase::IgnoreCase : ESearchCase::CaseSensitive;
	for (int32 ViewIndex = 0; ViewIndex != ViewLen;)
	{
		const TStringView<CharType> RemainingView(View.GetData() + ViewIndex, ViewLen - ViewIndex);
		auto MatchDelimiter = [RemainingView, SearchCase](TStringView<CharType> Delimiter) { return RemainingView.StartsWith(Delimiter, SearchCase); };
		if (const TStringView<CharType>* Delimiter = Algo::FindByPredicate(Delimiters, MatchDelimiter))
		{
			ParseTokensVisitToken(Visitor, Options, TStringView<CharType>(View.GetData() + NextTokenIndex, ViewIndex - NextTokenIndex));
			ViewIndex += Delimiter->Len();
			NextTokenIndex = ViewIndex;
		}
		else
		{
			++ViewIndex;
		}
	}

	ParseTokensVisitToken(Visitor, Options, TStringView<CharType>(View.GetData() + NextTokenIndex, ViewLen - NextTokenIndex));
}

template <typename CharType>
inline static void ParseTokensMultiple(
	const TStringView<CharType> View,
	const TConstArrayView<TStringView<CharType>> Delimiters,
	TFunctionRef<void (TStringView<CharType>)> Visitor,
	const EParseTokensOptions Options)
{
	check(Algo::NoneOf(Delimiters, &TStringView<CharType>::IsEmpty));
	switch (Delimiters.Num())
	{
	case 0:
		return ParseTokensVisitToken(Visitor, Options, View);
	case 1:
		if (Delimiters[0].Len() == 1)
		{
			return ParseTokens1Delim1Char(View, Delimiters[0][0], MoveTemp(Visitor), Options);
		}
		return ParseTokensNDelimNChar(View, Delimiters, MoveTemp(Visitor), Options);
	default:
		if (Algo::AllOf(Delimiters, [](const TStringView<CharType>& Delimiter) { return Delimiter.Len() == 1; }))
		{
			TArray<CharType, TInlineAllocator<32>> DelimiterChars;
			DelimiterChars.Reserve(Delimiters.Num());
			for (const TStringView<CharType>& Delimiter : Delimiters)
			{
				DelimiterChars.Add(Delimiter[0]);
			}
			return ParseTokensNDelim1Char<CharType>(View, DelimiterChars, MoveTemp(Visitor), Options);
		}
		else
		{
			return ParseTokensNDelimNChar(View, Delimiters, MoveTemp(Visitor), Options);
		}
	}
}

template <typename CharType>
inline static void ParseTokensMultiple(
	const TStringView<CharType> View,
	const TConstArrayView<CharType> Delimiters,
	TFunctionRef<void (TStringView<CharType>)> Visitor,
	const EParseTokensOptions Options)
{
	switch (Delimiters.Num())
	{
	case 0:
		return ParseTokensVisitToken(Visitor, Options, View);
	case 1:
		return ParseTokens1Delim1Char(View, Delimiters[0], MoveTemp(Visitor), Options);
	default:
		return ParseTokensNDelim1Char(View, Delimiters, MoveTemp(Visitor), Options);
	}
}

template <typename CharType>
inline static void ParseTokens(
	const TStringView<CharType> View,
	const TStringView<CharType> Delimiter,
	TFunctionRef<void (TStringView<CharType>)> Visitor,
	const EParseTokensOptions Options)
{
	if (Delimiter.Len() == 1)
	{
		return ParseTokens1Delim1Char(View, Delimiter[0], MoveTemp(Visitor), Options);
	}
	return ParseTokensNDelimNChar(View, MakeArrayView(&Delimiter, 1), MoveTemp(Visitor), Options);
}

} // UE::String::Private

namespace UE::String
{

void ParseTokens(const FAnsiStringView View, const ANSICHAR Delimiter, TFunctionRef<void (FAnsiStringView)> Visitor, const EParseTokensOptions Options)
{
	Private::ParseTokens1Delim1Char(View, Delimiter, MoveTemp(Visitor), Options);
}

void ParseTokens(const FWideStringView View, const WIDECHAR Delimiter, TFunctionRef<void (FWideStringView)> Visitor, const EParseTokensOptions Options)
{
	Private::ParseTokens1Delim1Char(View, Delimiter, MoveTemp(Visitor), Options);
}

void ParseTokens(const FUtf8StringView View, const UTF8CHAR Delimiter, TFunctionRef<void (FUtf8StringView)> Visitor, const EParseTokensOptions Options)
{
	Private::ParseTokens1Delim1Char(View, Delimiter, MoveTemp(Visitor), Options);
}

void ParseTokens(const FAnsiStringView View, const FAnsiStringView Delimiter, TFunctionRef<void (FAnsiStringView)> Visitor, const EParseTokensOptions Options)
{
	Private::ParseTokens(View, Delimiter, MoveTemp(Visitor), Options);
}

void ParseTokens(const FWideStringView View, const FWideStringView Delimiter, TFunctionRef<void (FWideStringView)> Visitor, const EParseTokensOptions Options)
{
	Private::ParseTokens(View, Delimiter, MoveTemp(Visitor), Options);
}

void ParseTokens(const FUtf8StringView View, const FUtf8StringView Delimiter, TFunctionRef<void (FUtf8StringView)> Visitor, const EParseTokensOptions Options)
{
	Private::ParseTokens(View, Delimiter, MoveTemp(Visitor), Options);
}

void ParseTokensMultiple(const FAnsiStringView View, const TConstArrayView<ANSICHAR> Delimiters, TFunctionRef<void (FAnsiStringView)> Visitor, const EParseTokensOptions Options)
{
	Private::ParseTokensMultiple(View, Delimiters, MoveTemp(Visitor), Options);
}

void ParseTokensMultiple(const FWideStringView View, const TConstArrayView<WIDECHAR> Delimiters, TFunctionRef<void (FWideStringView)> Visitor, const EParseTokensOptions Options)
{
	Private::ParseTokensMultiple(View, Delimiters, MoveTemp(Visitor), Options);
}

void ParseTokensMultiple(const FUtf8StringView View, const TConstArrayView<UTF8CHAR> Delimiters, TFunctionRef<void (FUtf8StringView)> Visitor, const EParseTokensOptions Options)
{
	Private::ParseTokensMultiple(View, Delimiters, MoveTemp(Visitor), Options);
}

void ParseTokensMultiple(const FAnsiStringView View, const TConstArrayView<FAnsiStringView> Delimiters, TFunctionRef<void (FAnsiStringView)> Visitor, const EParseTokensOptions Options)
{
	Private::ParseTokensMultiple(View, Delimiters, MoveTemp(Visitor), Options);
}

void ParseTokensMultiple(const FWideStringView View, const TConstArrayView<FWideStringView> Delimiters, TFunctionRef<void (FWideStringView)> Visitor, const EParseTokensOptions Options)
{
	Private::ParseTokensMultiple(View, Delimiters, MoveTemp(Visitor), Options);
}

void ParseTokensMultiple(const FUtf8StringView View, const TConstArrayView<FUtf8StringView> Delimiters, TFunctionRef<void (FUtf8StringView)> Visitor, const EParseTokensOptions Options)
{
	Private::ParseTokensMultiple(View, Delimiters, MoveTemp(Visitor), Options);
}

} // UE::String
