// Copyright Epic Games, Inc. All Rights Reserved.

#include "String/Split.h"

#include "Containers/StringView.h"
#include "String/Find.h"

namespace UE::String::Private
{

template <typename CharType>
static inline bool SplitOnIndex(TStringView<CharType> View, int32 Index, int32 Size, TStringView<CharType>& OutLeft, TStringView<CharType>& OutRight)
{
	if (Index == INDEX_NONE)
	{
		return false;
	}
	OutLeft = View.Left(Index);
	OutRight = View.RightChop(Index + Size);
	return true;
}

template <typename CharType>
static inline bool SplitFirst(TStringView<CharType> View, TStringView<CharType> Search, TStringView<CharType>& OutLeft, TStringView<CharType>& OutRight, ESearchCase::Type SearchCase)
{
	return SplitOnIndex(View, String::FindFirst(View, Search, SearchCase), Search.Len(), OutLeft, OutRight);
}

template <typename CharType>
static inline bool SplitLast(TStringView<CharType> View, TStringView<CharType> Search, TStringView<CharType>& OutLeft, TStringView<CharType>& OutRight, ESearchCase::Type SearchCase)
{
	return SplitOnIndex(View, String::FindLast(View, Search, SearchCase), Search.Len(), OutLeft, OutRight);
}

template <typename CharType>
static inline bool SplitFirstOfAny(TStringView<CharType> View, TConstArrayView<TStringView<CharType>> Search, TStringView<CharType>& OutLeft, TStringView<CharType>& OutRight, ESearchCase::Type SearchCase, int32* OutMatchIndex)
{
	int32 MatchIndex = INDEX_NONE;
	int32 ViewIndex = String::FindFirstOfAny(View, Search, SearchCase, &MatchIndex);
	int32 MatchSize = MatchIndex != INDEX_NONE ? Search[MatchIndex].Len() : 0;
	if (MatchIndex != INDEX_NONE && OutMatchIndex)
	{
		*OutMatchIndex = MatchIndex;
	}
	return SplitOnIndex(View, ViewIndex, MatchSize, OutLeft, OutRight);
}

template <typename CharType>
static inline bool SplitLastOfAny(TStringView<CharType> View, TConstArrayView<TStringView<CharType>> Search, TStringView<CharType>& OutLeft, TStringView<CharType>& OutRight, ESearchCase::Type SearchCase, int32* OutMatchIndex)
{
	int32 MatchIndex = INDEX_NONE;
	int32 ViewIndex = String::FindLastOfAny(View, Search, SearchCase, &MatchIndex);
	int32 MatchSize = MatchIndex != INDEX_NONE ? Search[MatchIndex].Len() : 0;
	if (MatchIndex != INDEX_NONE && OutMatchIndex)
	{
		*OutMatchIndex = MatchIndex;
	}
	return SplitOnIndex(View, ViewIndex, MatchSize, OutLeft, OutRight);
}

template <typename CharType>
static inline bool SplitFirstChar(TStringView<CharType> View, CharType Search, TStringView<CharType>& OutLeft, TStringView<CharType>& OutRight, ESearchCase::Type SearchCase)
{
	return SplitOnIndex(View, String::FindFirstChar(View, Search, SearchCase), 1, OutLeft, OutRight);
}

template <typename CharType>
static inline bool SplitLastChar(TStringView<CharType> View, CharType Search, TStringView<CharType>& OutLeft, TStringView<CharType>& OutRight, ESearchCase::Type SearchCase)
{
	return SplitOnIndex(View, String::FindLastChar(View, Search, SearchCase), 1, OutLeft, OutRight);
}

template <typename CharType>
static inline bool SplitFirstOfAnyChar(TStringView<CharType> View, TConstArrayView<CharType> Search, TStringView<CharType>& OutLeft, TStringView<CharType>& OutRight, ESearchCase::Type SearchCase, int32* OutMatchIndex)
{
	return SplitOnIndex(View, String::FindFirstOfAnyChar(View, Search, SearchCase, OutMatchIndex), 1, OutLeft, OutRight);
}

template <typename CharType>
static inline bool SplitLastOfAnyChar(TStringView<CharType> View, TConstArrayView<CharType> Search, TStringView<CharType>& OutLeft, TStringView<CharType>& OutRight, ESearchCase::Type SearchCase, int32* OutMatchIndex)
{
	return SplitOnIndex(View, String::FindLastOfAnyChar(View, Search, SearchCase, OutMatchIndex), 1, OutLeft, OutRight);
}

} // UE::String::Private

namespace UE::String
{

bool SplitFirst(FAnsiStringView View, FAnsiStringView Search, FAnsiStringView& OutLeft, FAnsiStringView& OutRight, ESearchCase::Type SearchCase)
{
	return Private::SplitFirst(View, Search, OutLeft, OutRight, SearchCase);
}

bool SplitFirst(FUtf8StringView View, FUtf8StringView Search, FUtf8StringView& OutLeft, FUtf8StringView& OutRight, ESearchCase::Type SearchCase)
{
	return Private::SplitFirst(View, Search, OutLeft, OutRight, SearchCase);
}

bool SplitFirst(FWideStringView View, FWideStringView Search, FWideStringView& OutLeft, FWideStringView& OutRight, ESearchCase::Type SearchCase)
{
	return Private::SplitFirst(View, Search, OutLeft, OutRight, SearchCase);
}

bool SplitLast(FAnsiStringView View, FAnsiStringView Search, FAnsiStringView& OutLeft, FAnsiStringView& OutRight, ESearchCase::Type SearchCase)
{
	return Private::SplitLast(View, Search, OutLeft, OutRight, SearchCase);
}

bool SplitLast(FUtf8StringView View, FUtf8StringView Search, FUtf8StringView& OutLeft, FUtf8StringView& OutRight, ESearchCase::Type SearchCase)
{
	return Private::SplitLast(View, Search, OutLeft, OutRight, SearchCase);
}

bool SplitLast(FWideStringView View, FWideStringView Search, FWideStringView& OutLeft, FWideStringView& OutRight, ESearchCase::Type SearchCase)
{
	return Private::SplitLast(View, Search, OutLeft, OutRight, SearchCase);
}

bool SplitFirstOfAny(FAnsiStringView View, TConstArrayView<FAnsiStringView> Search, FAnsiStringView& OutLeft, FAnsiStringView& OutRight, ESearchCase::Type SearchCase, int32* OutMatchIndex)
{
	return Private::SplitFirstOfAny(View, Search, OutLeft, OutRight, SearchCase, OutMatchIndex);
}

bool SplitFirstOfAny(FUtf8StringView View, TConstArrayView<FUtf8StringView> Search, FUtf8StringView& OutLeft, FUtf8StringView& OutRight, ESearchCase::Type SearchCase, int32* OutMatchIndex)
{
	return Private::SplitFirstOfAny(View, Search, OutLeft, OutRight, SearchCase, OutMatchIndex);
}

bool SplitFirstOfAny(FWideStringView View, TConstArrayView<FWideStringView> Search, FWideStringView& OutLeft, FWideStringView& OutRight, ESearchCase::Type SearchCase, int32* OutMatchIndex)
{
	return Private::SplitFirstOfAny(View, Search, OutLeft, OutRight, SearchCase, OutMatchIndex);
}

bool SplitLastOfAny(FAnsiStringView View, TConstArrayView<FAnsiStringView> Search, FAnsiStringView& OutLeft, FAnsiStringView& OutRight, ESearchCase::Type SearchCase, int32* OutMatchIndex)
{
	return Private::SplitLastOfAny(View, Search, OutLeft, OutRight, SearchCase, OutMatchIndex);
}

bool SplitLastOfAny(FUtf8StringView View, TConstArrayView<FUtf8StringView> Search, FUtf8StringView& OutLeft, FUtf8StringView& OutRight, ESearchCase::Type SearchCase, int32* OutMatchIndex)
{
	return Private::SplitLastOfAny(View, Search, OutLeft, OutRight, SearchCase, OutMatchIndex);
}

bool SplitLastOfAny(FWideStringView View, TConstArrayView<FWideStringView> Search, FWideStringView& OutLeft, FWideStringView& OutRight, ESearchCase::Type SearchCase, int32* OutMatchIndex)
{
	return Private::SplitLastOfAny(View, Search, OutLeft, OutRight, SearchCase, OutMatchIndex);
}

bool SplitFirstChar(FAnsiStringView View, ANSICHAR Search, FAnsiStringView& OutLeft, FAnsiStringView& OutRight, ESearchCase::Type SearchCase)
{
	return Private::SplitFirstChar(View, Search, OutLeft, OutRight, SearchCase);
}

bool SplitFirstChar(FUtf8StringView View, UTF8CHAR Search, FUtf8StringView& OutLeft, FUtf8StringView& OutRight, ESearchCase::Type SearchCase)
{
	return Private::SplitFirstChar(View, Search, OutLeft, OutRight, SearchCase);
}

bool SplitFirstChar(FWideStringView View, WIDECHAR Search, FWideStringView& OutLeft, FWideStringView& OutRight, ESearchCase::Type SearchCase)
{
	return Private::SplitFirstChar(View, Search, OutLeft, OutRight, SearchCase);
}

bool SplitLastChar(FAnsiStringView View, ANSICHAR Search, FAnsiStringView& OutLeft, FAnsiStringView& OutRight, ESearchCase::Type SearchCase)
{
	return Private::SplitLastChar(View, Search, OutLeft, OutRight, SearchCase);
}

bool SplitLastChar(FUtf8StringView View, UTF8CHAR Search, FUtf8StringView& OutLeft, FUtf8StringView& OutRight, ESearchCase::Type SearchCase)
{
	return Private::SplitLastChar(View, Search, OutLeft, OutRight, SearchCase);
}

bool SplitLastChar(FWideStringView View, WIDECHAR Search, FWideStringView& OutLeft, FWideStringView& OutRight, ESearchCase::Type SearchCase)
{
	return Private::SplitLastChar(View, Search, OutLeft, OutRight, SearchCase);
}

bool SplitFirstOfAnyChar(FAnsiStringView View, TConstArrayView<ANSICHAR> Search, FAnsiStringView& OutLeft, FAnsiStringView& OutRight, ESearchCase::Type SearchCase, int32* OutMatchIndex)
{
	return Private::SplitFirstOfAnyChar(View, Search, OutLeft, OutRight, SearchCase, OutMatchIndex);
}

bool SplitFirstOfAnyChar(FUtf8StringView View, TConstArrayView<UTF8CHAR> Search, FUtf8StringView& OutLeft, FUtf8StringView& OutRight, ESearchCase::Type SearchCase, int32* OutMatchIndex)
{
	return Private::SplitFirstOfAnyChar(View, Search, OutLeft, OutRight, SearchCase, OutMatchIndex);
}

bool SplitFirstOfAnyChar(FWideStringView View, TConstArrayView<WIDECHAR> Search, FWideStringView& OutLeft, FWideStringView& OutRight, ESearchCase::Type SearchCase, int32* OutMatchIndex)
{
	return Private::SplitFirstOfAnyChar(View, Search, OutLeft, OutRight, SearchCase, OutMatchIndex);
}

bool SplitLastOfAnyChar(FAnsiStringView View, TConstArrayView<ANSICHAR> Search, FAnsiStringView& OutLeft, FAnsiStringView& OutRight, ESearchCase::Type SearchCase, int32* OutMatchIndex)
{
	return Private::SplitLastOfAnyChar(View, Search, OutLeft, OutRight, SearchCase, OutMatchIndex);
}

bool SplitLastOfAnyChar(FUtf8StringView View, TConstArrayView<UTF8CHAR> Search, FUtf8StringView& OutLeft, FUtf8StringView& OutRight, ESearchCase::Type SearchCase, int32* OutMatchIndex)
{
	return Private::SplitLastOfAnyChar(View, Search, OutLeft, OutRight, SearchCase, OutMatchIndex);
}

bool SplitLastOfAnyChar(FWideStringView View, TConstArrayView<WIDECHAR> Search, FWideStringView& OutLeft, FWideStringView& OutRight, ESearchCase::Type SearchCase, int32* OutMatchIndex)
{
	return Private::SplitLastOfAnyChar(View, Search, OutLeft, OutRight, SearchCase, OutMatchIndex);
}

} // UE::String
