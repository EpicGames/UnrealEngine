// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tasks/Pipe.h"

#include "Containers/Array.h"
#include "Misc/ScopeExit.h"

namespace UE::Tasks
{
	Private::FTaskBase* FPipe::PushIntoPipe(Private::FTaskBase& Task)
	{
		Task.AddRef(); // the pipe holds a ref to the last task, until it's replaced by the next task or cleared on completion
		Private::FTaskBase* LastTask_Local = LastTask.exchange(&Task, std::memory_order_acq_rel); // `acq_rel` to order task construction before
		// its usage by a thread that replaces it as the last piped task
		checkf(LastTask_Local != &Task, TEXT("Dependency cycle: adding itself as a prerequisite (or use after destruction)"));

		if (LastTask_Local == nullptr)
		{
			return nullptr;
		}

		if (!LastTask_Local->AddSubsequent(Task))
		{
			// the last task doesn't accept subsequents anymore because it's already completed (happened concurrently after we replaced it as
			// the last pipe's task
			LastTask_Local->Release(); // the pipe doesn't need it anymore
			return nullptr;
		}

		return LastTask_Local; // transfer the reference to the caller that must release it
	}

	void FPipe::ClearTask(Private::FTaskBase& Task)
	{
		Private::FTaskBase* Task_Local = &Task;
		// try clearing the task if it's still pipe's "last task". if succeeded, release the ref accounted for pipe's last task. otherwise whoever replaced it
		// as the last task will do this

		// important to have a barrier even in case of failure so that whenever a pipe task finished, we have a barrier protecting any produced data
		// so that it can be passed across threads on the same pipe without synchronization.
		if (LastTask.compare_exchange_strong(Task_Local, nullptr, std::memory_order_acq_rel, std::memory_order_acquire))
		{
			Task.Release(); // it was still pipe's last task. now that we cleared it, release the reference
		}

		// Avoid use-after-free by taking a ref on the event before decrementing the value.
		// Since WaitUntilEmpty only looks at TaskCount to early out at which point
		// we could decide to get rid of the pipe object.
		TSharedRef<UE::FEventCount> LocalEmptyEvent = EmptyEventRef;
		if (TaskCount.fetch_sub(1, std::memory_order_release) == 1)
		{
			// use-after-free territory!

			LocalEmptyEvent->Notify();
		}
	}

	bool FPipe::WaitUntilEmpty(FTimespan InTimeout/* = FTimespan::MaxValue()*/)
	{
		if (TaskCount.load(std::memory_order_acquire) == 0)
		{
			return true;
		}

		TRACE_CPUPROFILER_EVENT_SCOPE(FPipe::WaitUntilEmpty);

		UE::FTimeout Timeout(InTimeout);
		while (true)
		{
			if (TaskCount.load(std::memory_order_acquire) == 0)
			{
				return true;
			}

			if (Timeout.IsExpired())
			{
				return false;
			}

			UE::FEventCountToken Token = EmptyEventRef->PrepareWait();

			if (TaskCount.load(std::memory_order_acquire) == 0)
			{
				return true;
			}

			if (!EmptyEventRef->WaitFor(Token, Timeout.WillNeverExpire() ? UE::FMonotonicTimeSpan::Infinity() : UE::FMonotonicTimeSpan::FromSeconds(Timeout.GetRemainingSeconds())))
			{
				break;
			}
		}

		return false;
	}

	// Maintains pipe callstack. Due to busy waiting tasks from multiple pipes can be executed nested.
	class FPipeCallStack
	{
	public:
		static void Push(const FPipe& Pipe)
		{
			CallStack.Add(&Pipe);
		}

		static void Pop(const FPipe& Pipe)
		{
			check(CallStack.Last() == &Pipe);
			CallStack.Pop(EAllowShrinking::No);
		}

		// returns true if a task from the given pipe is being being executed on the top of the stack.
		// the method deliberately doesn't look deeper because even if the pipe is there and technically it's safe to assume
		// accessing a resource protected by a pipe is thread-safe, logically it's a bug because it's an accidental condition
		static bool IsOnTop(const FPipe& Pipe)
		{
			return CallStack.Num() != 0 && CallStack.Last() == &Pipe;
		}

	private:
		static thread_local TArray<const FPipe*> CallStack;
	};

	thread_local TArray<const FPipe*> FPipeCallStack::CallStack;

	void FPipe::ExecutionStarted()
	{
		FPipeCallStack::Push(*this);
	}

	void FPipe::ExecutionFinished()
	{
		FPipeCallStack::Pop(*this);
	}

	bool FPipe::IsInContext() const
	{
		return FPipeCallStack::IsOnTop(*this);
	}
}
