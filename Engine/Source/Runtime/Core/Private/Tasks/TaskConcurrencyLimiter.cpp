// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tasks/TaskConcurrencyLimiter.h"
#include "Tasks/Task.h"
#include "Containers/LockFreeList.h"
#include "Templates/SharedPointer.h"
#include "Experimental/ConcurrentLinearAllocator.h"
#include "CoreTypes.h"

#include <AtomicQueue.h>

namespace UE::Tasks::TaskConcurrencyLimiter_Private
{

FPimpl::~FPimpl()
{
	if (FEvent* Event = CompletionEvent.exchange(nullptr, std::memory_order_relaxed))
	{
		FPlatformProcess::ReturnSynchEventToPool(Event);
	}
}

void FPimpl::AddWorkItem(FLimiterTask* Task)
{
	NumWorkItems.fetch_add(1, std::memory_order_acquire);

	WorkQueue.Push(Task);

	uint32 ConcurrencySlot;
	if (ConcurrencySlots.Alloc(ConcurrencySlot))
	{
		ProcessQueueFromPush(ConcurrencySlot);
	}
}

bool FPimpl::Wait(FTimespan Timeout)
{
	if (NumWorkItems.load(std::memory_order_relaxed) == 0)
	{
		return true;
	}

	FEvent* LocalCompletionEvent = CompletionEvent.load(std::memory_order_acquire);
	if (LocalCompletionEvent == nullptr)
	{
		FEvent* NewEvent = FPlatformProcess::GetSynchEventFromPool(true);
		if (!CompletionEvent.compare_exchange_strong(LocalCompletionEvent, NewEvent, std::memory_order_acq_rel, std::memory_order_acquire))
		{
			FPlatformProcess::ReturnSynchEventToPool(NewEvent);
		}
		else
		{
			LocalCompletionEvent = NewEvent;
		}
	}

	if (NumWorkItems.load(std::memory_order_acquire) == 0)
	{
		return true;
	}

	// We must use the global TaskScheduler's retraction feature (via the call to TryExpedite) to drain all tasks we
	// can find that are queued in the global TaskScheduler but not yet started, because at any time all of the
	// workerthreads in the global TaskScheduler might start working on tasks from another system (e.g. one calling
	// ParallelFor) and be unable to immediately work on the task that we launched. And if all of those tasks block on
	// a resource that our waitthread is holding (e.g. the AssetRegistry's internal critical section), then we would
	// deadlock. For simplicity we wait for a task to be launched into the global TaskScheduler before retracting the
	// task; this is a slight performance wastage but prevents us from having to duplicate or generalize the
	// ProcessQueue code to be called from here as well.
	bool bDidSomething;
	do
	{
		bDidSomething = false;
		for (int32 SlotIndex = 0; SlotIndex < ScheduledTasks.Num(); ++SlotIndex)
		{
			// If the task is being executed then the value will likely be null. However, it's possible the task is 
			// being executed but hasn't cleared this state yet. That is _fine_ since it means our expedite 
			// below will simply fail but it means we are now responsible for destroying the task and must call Release.
			if (FLimiterTask* LimiterTask = ScheduledTasks[SlotIndex].Task.exchange(nullptr, std::memory_order_acquire))
			{
				bDidSomething = true;
				// We populate ScheduledTasks before launching the task, as such we may have retracted a task
				// that hasn't launched yet. To handle that case we TryExecute, and if that fails 
				// (because we are already scheduled) we then expedite the task
				if (!LimiterTask->Task.TryExecute())
				{
					LimiterTask->Task.TryExpedite();
				}
				LimiterTask->Release(); 
			}
		}
	// Continue looking for retraction until all pushed items have completed. Stopping our search for retractables
	// as soon as we find nothing is retractable would potentially deadlock in the case that a workerthread afterwards
	// finishes our task that it was working on, queues our next task, but then is assigned another system's before it
	// can start working on the next task for us.
	} while (bDidSomething || NumWorkItems.load(std::memory_order_acquire) != 0);

	// This should fall-through immediately because NumWorkItems is 0 and so we called Trigger during the last
	// CompleteTask.
	return LocalCompletionEvent->Wait(Timeout);
}

void FPimpl::ProcessQueue(uint32 ConcurrencySlot, bool bSkipFirstWakeUp)
{
	bool bWakeUpWorker = !bSkipFirstWakeUp;
	do
	{
		if (FLimiterTask* LimiterTask = WorkQueue.Pop())
		{
			LowLevelTasks::FTask& Task = LimiterTask->Task;

			// Now that we know the ConcurrencySlot, set it at launch time so
			// the executor can retrieve it.
			Task.SetUserData((void*)(UPTRINT)ConcurrencySlot);

			// Increment the refcount and set the ScheduledTask. As soon as ScheduledTasks is set the task might 
			// be retracted, including before it has been scheduled here. As a result, TryLaunch may fail but that 
			// will only occur when TryExecute on the retracting thread has succeeded ensuring our task has been run.
			LimiterTask->AddRef();
			ScheduledTasks[ConcurrencySlot].Task.store(LimiterTask, std::memory_order_release); // Signal this task has now been scheduled
			LowLevelTasks::TryLaunch(Task, bWakeUpWorker ? LowLevelTasks::EQueuePreference::GlobalQueuePreference : LowLevelTasks::EQueuePreference::LocalQueuePreference, bWakeUpWorker);
		}
		else
		{
			ConcurrencySlots.Release(ConcurrencySlot);
			break;
		}

		// Don't skip wake-up if we launch any additional tasks.
		bWakeUpWorker = true;

	} while (ConcurrencySlots.Alloc(ConcurrencySlot));
}

void FPimpl::ProcessQueueFromWorker(uint32 ConcurrencySlot)
{
	// Once we are in a worker thread, we want to schedule on the local queue without waking up additional workers
	// to allow our own worker to pick up the next item and avoid wake-up cost / context switch.
	// We must check IsInWorkerThread() since it's possible we have retracted the work to run outside of a worker thread
	ProcessQueue(ConcurrencySlot, IsInWorkerThread());
}

void FPimpl::ProcessQueueFromPush(uint32 ConcurrencySlot)
{
	// When we push new items, we don't want to skip any wake-up.
	static constexpr bool bSkipFirstWakeUp = false;

	ProcessQueue(ConcurrencySlot, bSkipFirstWakeUp);
}

void FPimpl::CompleteWorkItem(uint32 ConcurrencySlot)
{
	if (NumWorkItems.fetch_sub(1, std::memory_order_release) == 1)
	{
		if (FEvent* LocalCompletionEvent = CompletionEvent.load(std::memory_order_acquire))
		{
			LocalCompletionEvent->Trigger();
		}
	}

	ProcessQueueFromWorker(ConcurrencySlot);
}

} // namespace UE::Tasks::TaskConcurrencyLimiter_Private
