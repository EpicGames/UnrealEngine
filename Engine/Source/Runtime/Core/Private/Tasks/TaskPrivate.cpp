// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tasks/TaskPrivate.h"
#include "Tasks/Pipe.h"

#include "Async/TaskGraphInterfaces.h"

CORE_API bool GTaskGraphAlwaysWaitWithNamedThreadSupport = 0;
static FAutoConsoleVariableRef CVarTaskGraphAlwaysWaitWithNamedThreadSupport(
	TEXT("TaskGraph.AlwaysWaitWithNamedThreadSupport"),
	GTaskGraphAlwaysWaitWithNamedThreadSupport,
	TEXT("Default to pumping the named thread tasks when waiting on named threads to avoid potential deadlocks."),
	ECVF_ReadOnly
);

namespace UE::Tasks
{
	namespace Private
	{
		// Specific translate functions, defined below
		ENamedThreads::Type TranslatePriority(EExtendedTaskPriority Priority);
		ENamedThreads::Type TranslatePriority(ETaskPriority Priority);
		
		FExecutableTaskAllocator SmallTaskAllocator;
		FTaskEventBaseAllocator TaskEventBaseAllocator;

		void FTaskBase::Schedule(bool& bWakeUpWorker)
		{
			TaskTrace::Scheduled(GetTraceId());

			if (IsNamedThreadTask())
			{
				FTaskGraphInterface::Get().QueueTask(static_cast<FBaseGraphTask*>(this), true, TranslatePriority(ExtendedPriority));
				return;
			}
			
			// In case a thread is waiting on us to perform retraction, now is the time to try retraction again.
			// This needs to be before the launch as performing the execution can destroy the task.
			StateChangeEvent.NotifyWeak();

			// This needs to be the last line touching any of the task's properties.
			bWakeUpWorker |= LowLevelTasks::FScheduler::Get().TryLaunch(LowLevelTask, bWakeUpWorker ? LowLevelTasks::EQueuePreference::GlobalQueuePreference : LowLevelTasks::EQueuePreference::LocalQueuePreference, bWakeUpWorker);

			// Use-after-free territory, do not touch any of the task's properties here.
		}

		thread_local uint32 TaskRetractionRecursion = 0;

		bool IsThreadRetractingTask()
		{
			return TaskRetractionRecursion != 0;
		}

		struct FThreadLocalRetractionScope
		{
			FThreadLocalRetractionScope()
			{
				checkSlow(TaskRetractionRecursion != TNumericLimits<decltype(TaskRetractionRecursion)>::Max() - 1);
				++TaskRetractionRecursion;
			}

			~FThreadLocalRetractionScope()
			{
				checkSlow(TaskRetractionRecursion != 0);
				--TaskRetractionRecursion;
			}
		};

		bool FTaskBase::TryRetractAndExecute(FTimeout Timeout, uint32 RecursionDepth/* = 0*/)
		{
			if (IsCompleted() || Timeout.IsExpired())
			{
				return IsCompleted();
			}

			TRACE_CPUPROFILER_EVENT_SCOPE(FTaskBase::TryRetractAndExecute);

			if (!IsAwaitable())
			{
				UE_LOGF(LogTemp, Fatal, "Deadlock detected! A task can't be waited here, e.g. because it's being executed by the current thread");
				return false;
			}

			if (IsNamedThreadTask())
			{
				if (ExtendedPriority >= EExtendedTaskPriority::GameThreadNormalPri &&
					ExtendedPriority <= EExtendedTaskPriority::GameThreadHiPriLocalQueue &&
					IsInGameThread())
				{
					// Allow task retraction of game thread tasks on the game thread
				}
				else if (ExtendedPriority >= EExtendedTaskPriority::RenderThreadNormalPri &&
					ExtendedPriority <= EExtendedTaskPriority::RenderThreadHiPriLocalQueue &&
					IsInRenderingThread())
				{
					// Allow task retraction of rendering tasks on the rendering thread
				}
				else if (ExtendedPriority >= EExtendedTaskPriority::RHIThreadNormalPri &&
					ExtendedPriority <= EExtendedTaskPriority::RHIThreadHiPriLocalQueue &&
					IsInRHIThread())
				{
					// Allow task retraction of RHI tasks on the RHI thread
				}
				else
				{
					return false;
				}
			}

			// avoid stack overflow. is not expected in a real-life cases but happens in stress tests
			if (RecursionDepth == 200)
			{
				return false;
			}
			++RecursionDepth;

			// returns false if the task has passed "pre-scheduling" state: all (if any) prerequisites are completed
			auto IsLockedByPrerequisites = [this]
			{
				uint32 LocalNumLocks = NumLocks.load(std::memory_order_relaxed); // the order doesn't matter as this "happens before" task execution
				return LocalNumLocks != 0 && LocalNumLocks < ExecutionFlag;
			};

			if (IsLockedByPrerequisites())
			{
				// try to unlock the task. even if (some or all) prerequisites retraction fails we still proceed to try helping with other prerequisites or this task execution

				// prerequisites are "consumed" here even if their retraction fails. this means that once prerequisite retraction failed, it won't be performed again. 
				// this can be potentially improved by using a different container for prerequisites
				for (FTaskBase* Prerequisite : Prerequisites.PopAll())
				{
					// ignore if retraction failed, as this thread still can try to help with other prerequisites instead of being blocked in waiting
					Prerequisite->TryRetractAndExecute(Timeout, RecursionDepth);
					Prerequisite->Release();
				}
			}

			// If we don't have any more prerequisites, let TryUnlock
			// execute these to avoid any race condition where we could clear
			// the last reference before TryUnlock finishes and cause a use-after-free.
			// These are super fast to process anyway so we can just consider them done
			// for retraction purpose.
			if (ExtendedPriority == EExtendedTaskPriority::TaskEvent ||
				ExtendedPriority == EExtendedTaskPriority::Inline)
			{
				return true;
			}

			if (Timeout.IsExpired())
			{
				return IsCompleted();
			}

			{
				FThreadLocalRetractionScope ThreadLocalRetractionScope;

				// next we try to execute the task, despite we haven't verified that the task is unlocked. trying to obtain execution permission will fail in this case

				if (!TryExecuteTask())
				{
					return false; // still locked by prerequisites, or another thread managed to set execution flag first, or we're inside this task execution
					// we could try to help with nested tasks execution (the task execution could already spawned a couple of nested tasks sitting in the queue). 
					// it's unclear how important this is, but this would definitely lead to more complicated impl. we can revisit this once we see such instances in profiler captures
				}
			}

			TRACE_CPUPROFILER_EVENT_SCOPE(SuccessfulTaskRetraction);

			// the task was launched so the scheduler will handle the internal reference held by low-level task

			// retract nested tasks, if any
			{
				// keep trying retracting all nested tasks even if some of them fail, so the current worker can contribute instead of being blocked
				bool bSucceeded = true;
				// prerequisites are "consumed" here even if their retraction fails. this means that once prerequisite retraction failed, it won't be performed again. 
				// this can be potentially improved by using a different container for prerequisites
				for (FTaskBase* Prerequisite : Prerequisites.PopAll())
				{
					if (!Prerequisite->TryRetractAndExecute(Timeout, RecursionDepth))
					{
						bSucceeded = false;
					}
					Prerequisite->Release();
				}

				if (!bSucceeded)
				{
					return false;
				}
			}

			// at this point the task is executed and has no pending nested tasks, but still can be "not completed" (nested tasks can be 
			// in the process of completing it (setting the flag) concurrently), so the caller still has to wait for completion
			return true;
		}

		bool FTaskBase::Wait(FTimeout Timeout)
		{
			if (IsCompleted() || Timeout.IsExpired())
			{
				return IsCompleted();
			}

			TaskTrace::FWaitingScope WaitingScope(GetTraceId());
			TRACE_CPUPROFILER_EVENT_SCOPE(Tasks::Wait);

			return WaitImpl(Timeout);
		}

		bool ShouldForceWaitWithNamedThreadsSupport(EExtendedTaskPriority Priority);
		void FTaskBase::Wait()
		{
			if (GTaskGraphAlwaysWaitWithNamedThreadSupport || ShouldForceWaitWithNamedThreadsSupport(ExtendedPriority))
			{
				WaitWithNamedThreadsSupport();
			}
			else
			{
				WaitImpl(FTimeout::Never());
			}
		}

		void FTaskBase::WaitWithNamedThreadsSupport()
		{
			if (IsCompleted())
			{
				return;
			}

			TRACE_CPUPROFILER_EVENT_SCOPE(FTaskBase::WaitWithNamedThreadsSupport);
			TaskTrace::FWaitingScope WaitingScope(GetTraceId());

			TryRetractAndExecute(FTimeout::Never());

			if (IsCompleted())
			{
				return;
			}

			if (!TryWaitOnNamedThread(*this))
			{
				WaitImpl(FTimeout::Never());
			}
		}

		bool FTaskBase::WaitImpl(FTimeout Timeout)
		{
			while (true)
			{
				// ignore the result as we still have to make sure the task is completed upon returning from this function call
				TryRetractAndExecute(Timeout);

				// spin for a while with hope the task is getting completed right now, to avoid getting blocked by a pricey syscall
				const uint32 MaxSpinCount = 40;
				for (uint32 SpinCount = 0; SpinCount != MaxSpinCount && !IsCompleted() && !Timeout.IsExpired(); ++SpinCount)
				{
					FPlatformProcess::Yield(); // YieldThread() was much slower on some platforms with low core count and contention for CPU
				}

				if (IsCompleted() || Timeout.IsExpired())
				{
					return IsCompleted();
				}

				auto Token = StateChangeEvent.PrepareWait();

				// Important to check the condition a second time after PrepareWait has been called to make sure we don't
				// miss an important state change event.
				if (IsCompleted())
				{
					return true;
				}

				{
					TRACE_CPUPROFILER_EVENT_SCOPE(FTaskBase::WaitImpl_StateChangeEvent_WaitFor);

					// Always flush events before entering a wait to make sure there's nothing missing in Unreal Insights that could prevent us understanding what's going on.
					TRACE_CPUPROFILER_EVENT_FLUSH();
					StateChangeEvent.WaitFor(Token, Timeout.WillNeverExpire() ? UE::FMonotonicTimeSpan::Infinity() : UE::FMonotonicTimeSpan::FromSeconds(Timeout.GetRemainingSeconds()));
				}

				// Once the state of the task has changed (either closed or scheduled), it's time to do another round of retraction to help if possible.
			}
		}

		FTaskBase* FTaskBase::TryPushIntoPipe()
		{
			return GetPipe()->PushIntoPipe(*this);
		}

		void FTaskBase::StartPipeExecution()
		{
			GetPipe()->ExecutionStarted();
		}

		void FTaskBase::FinishPipeExecution()
		{
			GetPipe()->ExecutionFinished();
		}

		void FTaskBase::ClearPipe()
		{
			GetPipe()->ClearTask(*this);
		}
	
		// Stack overflow prevention for deep Close -> TryUnlock -> Close chains.
		// Track recursion depth per-thread; beyond MaxCloseRecursionDepth, defer
		// subsequent unlocks into a worklist drained iteratively at the outermost level.
		static constexpr int32 MaxCloseRecursionDepth = 128;

		struct FCloseOverflowState
		{
			int32 Depth = 0;
			TArray<FTaskBase*> Pending;
		};
		static thread_local FCloseOverflowState CloseOverflow;

		void FTaskBase::Close()
		{
			TASKGRAPH_VERBOSE_EVENT_SCOPE(FTaskBase::Close);
			checkSlow(!IsCompleted());

			++CloseOverflow.Depth;

			// Push the first subsequent to the local queue so we pick it up directly as our next task.
			// This saves us the cost of going to the global queue and performing a wake-up.
			// But if we're a task event, always wake up new workers because the current task could continue executing for a long time after the trigger.
			bool bWakeUpWorker = ExtendedPriority == EExtendedTaskPriority::TaskEvent;

			auto LocalSubsequents = Subsequents.Close();

			if (LIKELY(CloseOverflow.Depth <= MaxCloseRecursionDepth))
			{
				for (FTaskBase* Subsequent : LocalSubsequents)
				{
					// bWakeUpWorker is passed by reference and is automatically set to true if we successfully schedule a task on the local queue.
					// so all the remaining ones are sent to the global queue.
					Subsequent->TryUnlock(bWakeUpWorker);
				}
			}
			else
			{
				for (FTaskBase* Subsequent : LocalSubsequents)
				{
					// Defer unlock to avoid stack overflow from deep recursion
					CloseOverflow.Pending.Add(Subsequent);
				}
			}

			// Clear the pipe after the task is completed (subsequents closed) so that any tasks part of the
			// pipe are not seen still being executed after FPipe::WaitUntilEmpty has returned.
			if (GetPipe() != nullptr)
			{
				ClearPipe();
			}

			// release nested tasks
			ReleasePrerequisites();

			TaskTrace::Completed(GetTraceId());

			// In case a thread is waiting on us to perform retraction, now is the time to try retraction again.
			StateChangeEvent.NotifyWeak();

			// The outermost Close on this thread drains any deferred unlocks iteratively.
			if (--CloseOverflow.Depth == 0 && CloseOverflow.Pending.Num() > 0)
			{
				// Always wake up workers, at this point we're long past the first element anyway
				bWakeUpWorker = true;

				TArray<FTaskBase*> Pending;

				// Bump Depth back to 1 during drain so recursive Close calls don't also try to drain.
				++CloseOverflow.Depth;
				while (CloseOverflow.Pending.Num())
				{
					// Backbuffer to maintain ordering without having to allocate for each iteration
					Swap(Pending, CloseOverflow.Pending);

					for (FTaskBase* Task : Pending)
					{
						// TryUnlock may recurse into Close, appending more items
						Task->TryUnlock(bWakeUpWorker);
					}

					Pending.Reset(); 
				}
				CloseOverflow.Pending.Empty(); // Don't leave an allocation when done
				--CloseOverflow.Depth;
			}
		}

		static thread_local FTaskBase* CurrentTask = nullptr;

		FTaskBase* GetCurrentTask()
		{
			return CurrentTask;
		}

		FTaskBase* ExchangeCurrentTask(FTaskBase* Task)
		{
			FTaskBase* PrevTask = CurrentTask;
			CurrentTask = Task;
			return PrevTask;
		}

		bool TryWaitOnNamedThread(FTaskBase& Task)
		{
			// handle waiting only on a named thread and if not called from inside a task
			FTaskGraphInterface& TaskGraph = FTaskGraphInterface::Get();
			ENamedThreads::Type CurrentThread = TaskGraph.GetCurrentThreadIfKnown();
			if (CurrentThread <= ENamedThreads::ActualRenderingThread /* is a named thread? */ && !TaskGraph.IsThreadProcessingTasks(CurrentThread))
			{
				// execute other tasks of this named thread while waiting
				ETaskPriority Dummy;
				EExtendedTaskPriority ExtendedPriority;
				TranslatePriority(CurrentThread, Dummy, ExtendedPriority);

				auto TaskBody = [CurrentThread, &TaskGraph] { TaskGraph.RequestReturn(CurrentThread); };
				using FReturnFromNamedThreadTask = TExecutableTask<decltype(TaskBody)>;
				FReturnFromNamedThreadTask ReturnTask { TEXT("ReturnFromNamedThreadTask"), MoveTemp(TaskBody), ETaskPriority::High, ExtendedPriority, ETaskFlags::None };
				ReturnTask.AddPrerequisites(Task);
				ReturnTask.TryLaunch(sizeof(ReturnTask)); // the result doesn't matter

				TaskGraph.ProcessThreadUntilRequestReturn(CurrentThread);
				check(Task.IsCompleted());
				return true;
			}

			return false;
		}
	}

	const TCHAR* ToString(EExtendedTaskPriority ExtendedPriority)
	{
		if (ExtendedPriority < EExtendedTaskPriority::None || ExtendedPriority >= EExtendedTaskPriority::Count)
		{
			return nullptr;
		}

		const TCHAR* ExtendedTaskPriorityToStr[] =
		{
			TEXT("None"),
			TEXT("Inline"),
			TEXT("TaskEvent"),

			TEXT("GameThreadNormalPri"),
			TEXT("GameThreadHiPri"),
			TEXT("GameThreadNormalPriLocalQueue"),
			TEXT("GameThreadHiPriLocalQueue"),

			TEXT("RenderThreadNormalPri"),
			TEXT("RenderThreadHiPri"),
			TEXT("RenderThreadNormalPriLocalQueue"),
			TEXT("RenderThreadHiPriLocalQueue"),

			TEXT("RHIThreadNormalPri"),
			TEXT("RHIThreadHiPri"),
			TEXT("RHIThreadNormalPriLocalQueue"),
			TEXT("RHIThreadHiPriLocalQueue")
		};
		return ExtendedTaskPriorityToStr[(int32)ExtendedPriority];
	}

	bool ToExtendedTaskPriority(const TCHAR* ExtendedPriorityStr, EExtendedTaskPriority& OutExtendedPriority)
	{
#define CONVERT_EXTENDED_TASK_PRIORITY(ExtendedTaskPriority)\
		if (FCString::Stricmp(ExtendedPriorityStr, ToString(EExtendedTaskPriority::ExtendedTaskPriority)) == 0)\
		{\
			OutExtendedPriority = EExtendedTaskPriority::ExtendedTaskPriority;\
			return true;\
		}

		CONVERT_EXTENDED_TASK_PRIORITY(None);
		CONVERT_EXTENDED_TASK_PRIORITY(Inline);
		CONVERT_EXTENDED_TASK_PRIORITY(TaskEvent);

		CONVERT_EXTENDED_TASK_PRIORITY(GameThreadNormalPri);
		CONVERT_EXTENDED_TASK_PRIORITY(GameThreadHiPri);
		CONVERT_EXTENDED_TASK_PRIORITY(GameThreadNormalPriLocalQueue);
		CONVERT_EXTENDED_TASK_PRIORITY(GameThreadHiPriLocalQueue);

		CONVERT_EXTENDED_TASK_PRIORITY(RenderThreadNormalPri);
		CONVERT_EXTENDED_TASK_PRIORITY(RenderThreadHiPri);
		CONVERT_EXTENDED_TASK_PRIORITY(RenderThreadNormalPriLocalQueue);
		CONVERT_EXTENDED_TASK_PRIORITY(RenderThreadHiPriLocalQueue);

		CONVERT_EXTENDED_TASK_PRIORITY(RHIThreadNormalPri);
		CONVERT_EXTENDED_TASK_PRIORITY(RHIThreadHiPri);
		CONVERT_EXTENDED_TASK_PRIORITY(RHIThreadNormalPriLocalQueue);
		CONVERT_EXTENDED_TASK_PRIORITY(RHIThreadHiPriLocalQueue);

#undef CONVERT_EXTENDED_TASK_PRIORITY

		return false;
	}

	FTaskPriorityCVar::FTaskPriorityCVar(const TCHAR* Name, const TCHAR* Help, ETaskPriority DefaultPriority, EExtendedTaskPriority DefaultExtendedPriority)
		: RawSetting(ConfigStringFromPriorities(DefaultPriority, DefaultExtendedPriority))
		, FullHelpText(CreateFullHelpText(Name, Help))
		, Variable(Name, RawSetting, *FullHelpText, FConsoleVariableDelegate::CreateRaw(this, &FTaskPriorityCVar::OnSettingChanged), ECVF_Default)
		, Priority(DefaultPriority)
		, ExtendedPriority(DefaultExtendedPriority)
	{
	}

	FString FTaskPriorityCVar::CreateFullHelpText(const TCHAR* Name, const TCHAR* OriginalHelp)
	{
		TStringBuilder<1024> TaskPriorities;
		for (int i = 0; i != (int)ETaskPriority::Count; ++i)
		{
			TaskPriorities.Append(ToString((ETaskPriority)i));
			TaskPriorities.Append(TEXT(", "));
		}
		TaskPriorities.RemoveSuffix(2); // remove the last ", "

		TStringBuilder<1024> ExtendedTaskPriorities;
		for (int i = 0; i != (int)EExtendedTaskPriority::Count; ++i)
		{
			ExtendedTaskPriorities.Append(ToString((EExtendedTaskPriority)i));
			ExtendedTaskPriorities.Append(TEXT(", "));
		}
		ExtendedTaskPriorities.RemoveSuffix(2); // remove the last ", "

		return FString::Printf(
			TEXT("%s\n")
			TEXT("Arguments are task priority and extended task priority (optional) separated by a space: [TaskPriority] [ExtendedTaskPriority]\n")
			TEXT("where TaskPriority is in [%s]\n")
			TEXT("and ExtendedTaskPriority is in [%s].\n")
			TEXT("Example: \"%s %s %s\" or \"%s\"")
			, OriginalHelp, *TaskPriorities, *ExtendedTaskPriorities, Name, ToString((ETaskPriority)0), ToString((EExtendedTaskPriority)0), ToString((ETaskPriority)0));
	}

	FString FTaskPriorityCVar::ConfigStringFromPriorities(ETaskPriority InPriority, EExtendedTaskPriority InExtendedPriority)
	{
		return FString{ ToString(InPriority) } + TEXT(" ") + ToString(InExtendedPriority);
	}

	void FTaskPriorityCVar::OnSettingChanged(IConsoleVariable* InVariable)
	{
		FString PriorityStr, ExtendedPriorityStr;
		static FString Delimiter{ " " };
		if (RawSetting.Split(Delimiter, &PriorityStr, &ExtendedPriorityStr))
		{
			verify(ToTaskPriority(*PriorityStr, Priority));
			verify(ToExtendedTaskPriority(*ExtendedPriorityStr, ExtendedPriority));
		}
		else
		{
			verify(ToTaskPriority(*RawSetting, Priority));
			ExtendedPriority = EExtendedTaskPriority::None;
		}
	}

	namespace Private
	{
		// task priority translation from the old API to the new API
		void TranslatePriority(ENamedThreads::Type ThreadType, ETaskPriority& OutPriority, EExtendedTaskPriority& OutExtendedPriority)
		{
			using namespace UE::Tasks;

			ENamedThreads::Type ThreadIndex = ENamedThreads::GetThreadIndex(ThreadType);
			if (ThreadIndex != ENamedThreads::AnyThread)
			{
				check(ThreadIndex == ENamedThreads::GameThread || ThreadIndex == ENamedThreads::ActualRenderingThread || ThreadIndex == ENamedThreads::RHIThread);
				EExtendedTaskPriority ConversionMap[] =
				{
					EExtendedTaskPriority::RHIThreadNormalPri,
					EExtendedTaskPriority::GameThreadNormalPri,
					EExtendedTaskPriority::RenderThreadNormalPri
				};
				OutExtendedPriority = ConversionMap[ThreadIndex - ENamedThreads::RHIThread];
				OutExtendedPriority = (EExtendedTaskPriority)((int32)OutExtendedPriority + (ENamedThreads::GetTaskPriority(ThreadType) != ENamedThreads::NormalTaskPriority ? 1 : 0));
				OutExtendedPriority = (EExtendedTaskPriority)((int32)OutExtendedPriority + (ENamedThreads::GetQueueIndex(ThreadType) != ENamedThreads::MainQueue ? 2 : 0));
				OutPriority = ETaskPriority::Count;
			}
			else
			{
				OutExtendedPriority = EExtendedTaskPriority::None;
				uint32 ThreadPriority = GetThreadPriorityIndex(ThreadType);
				check(ThreadPriority < uint32(ENamedThreads::NumThreadPriorities));
				ETaskPriority ConversionMap[int(ENamedThreads::NumThreadPriorities)] = { ETaskPriority::Normal, ETaskPriority::High, ETaskPriority::BackgroundNormal };
				OutPriority = ConversionMap[ThreadPriority];
			}

			if (OutPriority == ETaskPriority::BackgroundNormal && GetTaskPriority(ThreadType))
			{
				OutPriority = ETaskPriority::BackgroundHigh;
			}
		}

		ENamedThreads::Type TranslatePriority(ETaskPriority Priority)
		{
			checkSlow(Priority < ETaskPriority::Count);

			ENamedThreads::Type ConversionMap[] =
			{
				ENamedThreads::AnyHiPriThreadNormalTask,
				ENamedThreads::AnyNormalThreadNormalTask,
				ENamedThreads::AnyBackgroundHiPriTask,
				ENamedThreads::AnyBackgroundThreadNormalTask,
				ENamedThreads::AnyBackgroundThreadNormalTask // same as above
			};

			return ConversionMap[(int32)Priority];
		}

		// task priority translation from the new API to the old API
		ENamedThreads::Type TranslatePriority(EExtendedTaskPriority Priority)
		{
			// Switch is faster than table because of render thread
			switch (Priority)
			{
			case EExtendedTaskPriority::GameThreadNormalPri:
				return ENamedThreads::GameThread;
			case EExtendedTaskPriority::GameThreadHiPri:
				return (ENamedThreads::Type)(ENamedThreads::GameThread | ENamedThreads::HighTaskPriority);
			case EExtendedTaskPriority::GameThreadNormalPriLocalQueue:
				return (ENamedThreads::Type)(ENamedThreads::GameThread | ENamedThreads::LocalQueue);
			case EExtendedTaskPriority::GameThreadHiPriLocalQueue:
				return (ENamedThreads::Type)(ENamedThreads::GameThread | ENamedThreads::HighTaskPriority | ENamedThreads::LocalQueue);

			case EExtendedTaskPriority::RenderThreadNormalPri:
				return ENamedThreads::GetRenderThread();
			case EExtendedTaskPriority::RenderThreadHiPri:
				return (ENamedThreads::Type)(ENamedThreads::GetRenderThread() | ENamedThreads::HighTaskPriority);
			case EExtendedTaskPriority::RenderThreadNormalPriLocalQueue:
				return (ENamedThreads::Type)(ENamedThreads::GetRenderThread() | ENamedThreads::LocalQueue);
			case EExtendedTaskPriority::RenderThreadHiPriLocalQueue:
				return (ENamedThreads::Type)(ENamedThreads::GetRenderThread() | ENamedThreads::HighTaskPriority | ENamedThreads::LocalQueue);

			case EExtendedTaskPriority::RHIThreadNormalPri:
				return ENamedThreads::RHIThread;
			case EExtendedTaskPriority::RHIThreadHiPri:
				return (ENamedThreads::Type)(ENamedThreads::RHIThread | ENamedThreads::HighTaskPriority);
			case EExtendedTaskPriority::RHIThreadNormalPriLocalQueue:
				return (ENamedThreads::Type)(ENamedThreads::RHIThread | ENamedThreads::LocalQueue);
			case EExtendedTaskPriority::RHIThreadHiPriLocalQueue:
				return (ENamedThreads::Type)(ENamedThreads::RHIThread | ENamedThreads::HighTaskPriority | ENamedThreads::LocalQueue);

			default:
				checkf(Priority >= EExtendedTaskPriority::GameThreadNormalPri && Priority < EExtendedTaskPriority::Count, TEXT("only named threads can call this method: %d"), int32{EnumToUnderlyingType(Priority)});
				return ENamedThreads::AnyThread;
			}
		}

		ENamedThreads::Type TranslatePriority(ETaskPriority Priority, EExtendedTaskPriority ExtendedPriority)
		{
			checkSlow(Priority <= ETaskPriority::Count);
			checkSlow(ExtendedPriority <= EExtendedTaskPriority::Count);

			return ExtendedPriority < EExtendedTaskPriority::GameThreadNormalPri || ExtendedPriority == EExtendedTaskPriority::Count ? TranslatePriority(Priority) : TranslatePriority(ExtendedPriority);
		}

		bool ShouldForceWaitWithNamedThreadsSupport(EExtendedTaskPriority ExtendedPriority)
		{
			// We force wait named thread support when we're waiting on a task that must run on the same thread we're currently on.
			// If we don't do this, it's a guaranteed deadlock.
			const bool bIsNamedThreadTask = ExtendedPriority >= EExtendedTaskPriority::GameThreadNormalPri;
			if (bIsNamedThreadTask)
			{
				FTaskGraphInterface& TaskGraph = FTaskGraphInterface::Get();
				ENamedThreads::Type CurrentThreadIndex = ENamedThreads::GetThreadIndex(TaskGraph.GetCurrentThreadIfKnown());
				if (CurrentThreadIndex <= ENamedThreads::ActualRenderingThread)
				{
					ENamedThreads::Type TaskThreadIndex = ENamedThreads::GetThreadIndex(TranslatePriority(ExtendedPriority));
					return TaskThreadIndex == CurrentThreadIndex;
				}
			}
			return false;
		}

		static thread_local FCancellationToken* CancellationToken = nullptr;
	}

	void FCancellationTokenScope::SetToken(FCancellationToken* CancellationToken)
	{
		if (CancellationToken)
		{
			if (Private::CancellationToken != CancellationToken)
			{
				check(Private::CancellationToken == nullptr);
				Private::CancellationToken = CancellationToken;
				bHasActiveScope = true;
			}
		}
	}

	FCancellationTokenScope::FCancellationTokenScope(FCancellationToken& CancellationToken)
	{
		SetToken(&CancellationToken);
	}

	FCancellationTokenScope::FCancellationTokenScope(FCancellationToken* CancellationToken)
	{
		SetToken(CancellationToken);
	}

	FCancellationTokenScope::~FCancellationTokenScope()
	{
		if (bHasActiveScope)
		{
			Private::CancellationToken = nullptr;
		}
	}

	FCancellationToken* FCancellationTokenScope::GetCurrentCancellationToken()
	{
		return Private::CancellationToken;
	}
}
