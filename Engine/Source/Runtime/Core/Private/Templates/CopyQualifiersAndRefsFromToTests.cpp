// Copyright Epic Games, Inc. All Rights Reserved.

#include <type_traits>
#include "Templates/CopyQualifiersAndRefsFromTo.h"

static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<               int  ,                int  >,                int  >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<const          int  ,                int  >, const          int  >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<      volatile int  ,                int  >,       volatile int  >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<const volatile int  ,                int  >, const volatile int  >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<               int& ,                int  >,                int& >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<const          int& ,                int  >, const          int& >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<      volatile int& ,                int  >,       volatile int& >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<const volatile int& ,                int  >, const volatile int& >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<               int&&,                int  >,                int&&>::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<const          int&&,                int  >, const          int&&>::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<      volatile int&&,                int  >,       volatile int&&>::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<const volatile int&&,                int  >, const volatile int&&>::value, "");

static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<               int  , const          int  >, const          int  >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<const          int  , const          int  >, const          int  >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<      volatile int  , const          int  >, const volatile int  >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<const volatile int  , const          int  >, const volatile int  >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<               int& , const          int  >, const          int& >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<const          int& , const          int  >, const          int& >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<      volatile int& , const          int  >, const volatile int& >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<const volatile int& , const          int  >, const volatile int& >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<               int&&, const          int  >, const          int&&>::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<const          int&&, const          int  >, const          int&&>::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<      volatile int&&, const          int  >, const volatile int&&>::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<const volatile int&&, const          int  >, const volatile int&&>::value, "");

static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<               int  ,       volatile int  >,       volatile int  >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<const          int  ,       volatile int  >, const volatile int  >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<      volatile int  ,       volatile int  >,       volatile int  >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<const volatile int  ,       volatile int  >, const volatile int  >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<               int& ,       volatile int  >,       volatile int& >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<const          int& ,       volatile int  >, const volatile int& >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<      volatile int& ,       volatile int  >,       volatile int& >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<const volatile int& ,       volatile int  >, const volatile int& >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<               int&&,       volatile int  >,       volatile int&&>::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<const          int&&,       volatile int  >, const volatile int&&>::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<      volatile int&&,       volatile int  >,       volatile int&&>::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<const volatile int&&,       volatile int  >, const volatile int&&>::value, "");

static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<               int  , const volatile int  >, const volatile int  >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<const          int  , const volatile int  >, const volatile int  >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<      volatile int  , const volatile int  >, const volatile int  >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<const volatile int  , const volatile int  >, const volatile int  >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<               int& , const volatile int  >, const volatile int& >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<const          int& , const volatile int  >, const volatile int& >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<      volatile int& , const volatile int  >, const volatile int& >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<const volatile int& , const volatile int  >, const volatile int& >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<               int&&, const volatile int  >, const volatile int&&>::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<const          int&&, const volatile int  >, const volatile int&&>::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<      volatile int&&, const volatile int  >, const volatile int&&>::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<const volatile int&&, const volatile int  >, const volatile int&&>::value, "");

static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<               int  ,                int& >,                int& >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<const          int  ,                int& >, const          int& >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<      volatile int  ,                int& >,       volatile int& >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<const volatile int  ,                int& >, const volatile int& >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<               int& ,                int& >,                int& >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<const          int& ,                int& >, const          int& >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<      volatile int& ,                int& >,       volatile int& >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<const volatile int& ,                int& >, const volatile int& >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<               int&&,                int& >,                int& >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<const          int&&,                int& >, const          int& >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<      volatile int&&,                int& >,       volatile int& >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<const volatile int&&,                int& >, const volatile int& >::value, "");

static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<               int  , const          int& >, const          int& >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<const          int  , const          int& >, const          int& >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<      volatile int  , const          int& >, const volatile int& >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<const volatile int  , const          int& >, const volatile int& >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<               int& , const          int& >, const          int& >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<const          int& , const          int& >, const          int& >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<      volatile int& , const          int& >, const volatile int& >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<const volatile int& , const          int& >, const volatile int& >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<               int&&, const          int& >, const          int& >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<const          int&&, const          int& >, const          int& >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<      volatile int&&, const          int& >, const volatile int& >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<const volatile int&&, const          int& >, const volatile int& >::value, "");

static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<               int  ,       volatile int& >,       volatile int& >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<const          int  ,       volatile int& >, const volatile int& >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<      volatile int  ,       volatile int& >,       volatile int& >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<const volatile int  ,       volatile int& >, const volatile int& >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<               int& ,       volatile int& >,       volatile int& >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<const          int& ,       volatile int& >, const volatile int& >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<      volatile int& ,       volatile int& >,       volatile int& >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<const volatile int& ,       volatile int& >, const volatile int& >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<               int&&,       volatile int& >,       volatile int& >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<const          int&&,       volatile int& >, const volatile int& >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<      volatile int&&,       volatile int& >,       volatile int& >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<const volatile int&&,       volatile int& >, const volatile int& >::value, "");

static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<               int  ,                int&&>,                int&&>::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<const          int  ,                int&&>, const          int&&>::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<      volatile int  ,                int&&>,       volatile int&&>::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<const volatile int  ,                int&&>, const volatile int&&>::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<               int& ,                int&&>,                int& >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<const          int& ,                int&&>, const          int& >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<      volatile int& ,                int&&>,       volatile int& >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<const volatile int& ,                int&&>, const volatile int& >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<               int&&,                int&&>,                int&&>::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<const          int&&,                int&&>, const          int&&>::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<      volatile int&&,                int&&>,       volatile int&&>::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<const volatile int&&,                int&&>, const volatile int&&>::value, "");

static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<               int  , const          int&&>, const          int&&>::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<const          int  , const          int&&>, const          int&&>::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<      volatile int  , const          int&&>, const volatile int&&>::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<const volatile int  , const          int&&>, const volatile int&&>::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<               int& , const          int&&>, const          int& >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<const          int& , const          int&&>, const          int& >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<      volatile int& , const          int&&>, const volatile int& >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<const volatile int& , const          int&&>, const volatile int& >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<               int&&, const          int&&>, const          int&&>::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<const          int&&, const          int&&>, const          int&&>::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<      volatile int&&, const          int&&>, const volatile int&&>::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<const volatile int&&, const          int&&>, const volatile int&&>::value, "");

static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<               int  ,       volatile int&&>,       volatile int&&>::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<const          int  ,       volatile int&&>, const volatile int&&>::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<      volatile int  ,       volatile int&&>,       volatile int&&>::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<const volatile int  ,       volatile int&&>, const volatile int&&>::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<               int& ,       volatile int&&>,       volatile int& >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<const          int& ,       volatile int&&>, const volatile int& >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<      volatile int& ,       volatile int&&>,       volatile int& >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<const volatile int& ,       volatile int&&>, const volatile int& >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<               int&&,       volatile int&&>,       volatile int&&>::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<const          int&&,       volatile int&&>, const volatile int&&>::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<      volatile int&&,       volatile int&&>,       volatile int&&>::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<const volatile int&&,       volatile int&&>, const volatile int&&>::value, "");

static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<               int  , const volatile int&&>, const volatile int&&>::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<const          int  , const volatile int&&>, const volatile int&&>::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<      volatile int  , const volatile int&&>, const volatile int&&>::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<const volatile int  , const volatile int&&>, const volatile int&&>::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<               int& , const volatile int&&>, const volatile int& >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<const          int& , const volatile int&&>, const volatile int& >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<      volatile int& , const volatile int&&>, const volatile int& >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<const volatile int& , const volatile int&&>, const volatile int& >::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<const          int&&, const volatile int&&>, const volatile int&&>::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<      volatile int&&, const volatile int&&>, const volatile int&&>::value, "");
static_assert(std::is_same<TCopyQualifiersAndRefsFromTo_T<const volatile int&&, const volatile int&&>, const volatile int&&>::value, "");
