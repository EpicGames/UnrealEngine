// Copyright Epic Games, Inc. All Rights Reserved.

#include <type_traits>
#include <utility>
#include "Templates/UnrealTemplate.h"

// Check that our ForwardLike has the same semantics as the example on https://en.cppreference.com/w/cpp/utility/forward_like.html
namespace UE::Tests
{
	template <typename T>
	constexpr std::add_volatile_t<T>& as_volatile(T& t) noexcept
	{
		return t;
	}

	template <class T, class U>
	constexpr auto&& forward_like(U&& x) noexcept
	{
		// std::forward_like doesn't support volatile, so add it here.

		constexpr bool is_adding_const = std::is_const_v<std::remove_reference_t<T>>;
		constexpr bool is_adding_volatile = std::is_volatile_v<std::remove_reference_t<T>>;
		if constexpr (std::is_lvalue_reference_v<T&&>)
		{
			if constexpr (is_adding_const)
			{
				if constexpr (is_adding_volatile)
				{
					return as_volatile(std::as_const(x));
				}
				else
				{
					return std::as_const(x);
				}
			}
			else
			{
				if constexpr (is_adding_volatile)
				{
					return as_volatile(static_cast<U&>(x));
				}
				else
				{
					return static_cast<U&>(x);
				}
			}
		}
		else
		{
			if constexpr (is_adding_const)
			{
				if constexpr (is_adding_volatile)
				{
					return std::move(as_volatile(std::as_const(x)));
				}
				else
				{
					return std::move(std::as_const(x));
				}
			}
			else
			{
				if constexpr (is_adding_volatile)
				{
					return std::move(as_volatile(x));
				}
				else
				{
					return std::move(x);
				}
			}
		}
	}
}

template <typename Type, typename ForwardLikeType>
static inline constexpr bool bForwardLikeMatchesStd_V =
	std::is_same_v<
		decltype(UE::Tests::forward_like<ForwardLikeType>(std::declval<Type>())),
		decltype(ForwardLike<ForwardLikeType>(std::declval<Type>()))
	>;

struct FForwardLikeInnerType
{
};

struct FForwardLikeOuterType
{
	FForwardLikeInnerType Inner;
};

static_assert(bForwardLikeMatchesStd_V<               FForwardLikeInnerType  ,                FForwardLikeOuterType  >, "");
static_assert(bForwardLikeMatchesStd_V<const          FForwardLikeInnerType  ,                FForwardLikeOuterType  >, "");
static_assert(bForwardLikeMatchesStd_V<      volatile FForwardLikeInnerType  ,                FForwardLikeOuterType  >, "");
static_assert(bForwardLikeMatchesStd_V<const volatile FForwardLikeInnerType  ,                FForwardLikeOuterType  >, "");
static_assert(bForwardLikeMatchesStd_V<               FForwardLikeInnerType& ,                FForwardLikeOuterType  >, "");
static_assert(bForwardLikeMatchesStd_V<const          FForwardLikeInnerType& ,                FForwardLikeOuterType  >, "");
static_assert(bForwardLikeMatchesStd_V<      volatile FForwardLikeInnerType& ,                FForwardLikeOuterType  >, "");
static_assert(bForwardLikeMatchesStd_V<const volatile FForwardLikeInnerType& ,                FForwardLikeOuterType  >, "");
static_assert(bForwardLikeMatchesStd_V<               FForwardLikeInnerType&&,                FForwardLikeOuterType  >, "");
static_assert(bForwardLikeMatchesStd_V<const          FForwardLikeInnerType&&,                FForwardLikeOuterType  >, "");
static_assert(bForwardLikeMatchesStd_V<      volatile FForwardLikeInnerType&&,                FForwardLikeOuterType  >, "");
static_assert(bForwardLikeMatchesStd_V<const volatile FForwardLikeInnerType&&,                FForwardLikeOuterType  >, "");

static_assert(bForwardLikeMatchesStd_V<               FForwardLikeInnerType  , const          FForwardLikeOuterType  >, "");
static_assert(bForwardLikeMatchesStd_V<const          FForwardLikeInnerType  , const          FForwardLikeOuterType  >, "");
static_assert(bForwardLikeMatchesStd_V<      volatile FForwardLikeInnerType  , const          FForwardLikeOuterType  >, "");
static_assert(bForwardLikeMatchesStd_V<const volatile FForwardLikeInnerType  , const          FForwardLikeOuterType  >, "");
static_assert(bForwardLikeMatchesStd_V<               FForwardLikeInnerType& , const          FForwardLikeOuterType  >, "");
static_assert(bForwardLikeMatchesStd_V<const          FForwardLikeInnerType& , const          FForwardLikeOuterType  >, "");
static_assert(bForwardLikeMatchesStd_V<      volatile FForwardLikeInnerType& , const          FForwardLikeOuterType  >, "");
static_assert(bForwardLikeMatchesStd_V<const volatile FForwardLikeInnerType& , const          FForwardLikeOuterType  >, "");
static_assert(bForwardLikeMatchesStd_V<               FForwardLikeInnerType&&, const          FForwardLikeOuterType  >, "");
static_assert(bForwardLikeMatchesStd_V<const          FForwardLikeInnerType&&, const          FForwardLikeOuterType  >, "");
static_assert(bForwardLikeMatchesStd_V<      volatile FForwardLikeInnerType&&, const          FForwardLikeOuterType  >, "");
static_assert(bForwardLikeMatchesStd_V<const volatile FForwardLikeInnerType&&, const          FForwardLikeOuterType  >, "");

static_assert(bForwardLikeMatchesStd_V<               FForwardLikeInnerType  ,       volatile FForwardLikeOuterType  >, "");
static_assert(bForwardLikeMatchesStd_V<const          FForwardLikeInnerType  ,       volatile FForwardLikeOuterType  >, "");
static_assert(bForwardLikeMatchesStd_V<      volatile FForwardLikeInnerType  ,       volatile FForwardLikeOuterType  >, "");
static_assert(bForwardLikeMatchesStd_V<const volatile FForwardLikeInnerType  ,       volatile FForwardLikeOuterType  >, "");
static_assert(bForwardLikeMatchesStd_V<               FForwardLikeInnerType& ,       volatile FForwardLikeOuterType  >, "");
static_assert(bForwardLikeMatchesStd_V<const          FForwardLikeInnerType& ,       volatile FForwardLikeOuterType  >, "");
static_assert(bForwardLikeMatchesStd_V<      volatile FForwardLikeInnerType& ,       volatile FForwardLikeOuterType  >, "");
static_assert(bForwardLikeMatchesStd_V<const volatile FForwardLikeInnerType& ,       volatile FForwardLikeOuterType  >, "");
static_assert(bForwardLikeMatchesStd_V<               FForwardLikeInnerType&&,       volatile FForwardLikeOuterType  >, "");
static_assert(bForwardLikeMatchesStd_V<const          FForwardLikeInnerType&&,       volatile FForwardLikeOuterType  >, "");
static_assert(bForwardLikeMatchesStd_V<      volatile FForwardLikeInnerType&&,       volatile FForwardLikeOuterType  >, "");
static_assert(bForwardLikeMatchesStd_V<const volatile FForwardLikeInnerType&&,       volatile FForwardLikeOuterType  >, "");

static_assert(bForwardLikeMatchesStd_V<               FForwardLikeInnerType  , const volatile FForwardLikeOuterType  >, "");
static_assert(bForwardLikeMatchesStd_V<const          FForwardLikeInnerType  , const volatile FForwardLikeOuterType  >, "");
static_assert(bForwardLikeMatchesStd_V<      volatile FForwardLikeInnerType  , const volatile FForwardLikeOuterType  >, "");
static_assert(bForwardLikeMatchesStd_V<const volatile FForwardLikeInnerType  , const volatile FForwardLikeOuterType  >, "");
static_assert(bForwardLikeMatchesStd_V<               FForwardLikeInnerType& , const volatile FForwardLikeOuterType  >, "");
static_assert(bForwardLikeMatchesStd_V<const          FForwardLikeInnerType& , const volatile FForwardLikeOuterType  >, "");
static_assert(bForwardLikeMatchesStd_V<      volatile FForwardLikeInnerType& , const volatile FForwardLikeOuterType  >, "");
static_assert(bForwardLikeMatchesStd_V<const volatile FForwardLikeInnerType& , const volatile FForwardLikeOuterType  >, "");
static_assert(bForwardLikeMatchesStd_V<               FForwardLikeInnerType&&, const volatile FForwardLikeOuterType  >, "");
static_assert(bForwardLikeMatchesStd_V<const          FForwardLikeInnerType&&, const volatile FForwardLikeOuterType  >, "");
static_assert(bForwardLikeMatchesStd_V<      volatile FForwardLikeInnerType&&, const volatile FForwardLikeOuterType  >, "");
static_assert(bForwardLikeMatchesStd_V<const volatile FForwardLikeInnerType&&, const volatile FForwardLikeOuterType  >, "");

static_assert(bForwardLikeMatchesStd_V<               FForwardLikeInnerType  ,                FForwardLikeOuterType& >, "");
static_assert(bForwardLikeMatchesStd_V<const          FForwardLikeInnerType  ,                FForwardLikeOuterType& >, "");
static_assert(bForwardLikeMatchesStd_V<      volatile FForwardLikeInnerType  ,                FForwardLikeOuterType& >, "");
static_assert(bForwardLikeMatchesStd_V<const volatile FForwardLikeInnerType  ,                FForwardLikeOuterType& >, "");
static_assert(bForwardLikeMatchesStd_V<               FForwardLikeInnerType& ,                FForwardLikeOuterType& >, "");
static_assert(bForwardLikeMatchesStd_V<const          FForwardLikeInnerType& ,                FForwardLikeOuterType& >, "");
static_assert(bForwardLikeMatchesStd_V<      volatile FForwardLikeInnerType& ,                FForwardLikeOuterType& >, "");
static_assert(bForwardLikeMatchesStd_V<const volatile FForwardLikeInnerType& ,                FForwardLikeOuterType& >, "");
static_assert(bForwardLikeMatchesStd_V<               FForwardLikeInnerType&&,                FForwardLikeOuterType& >, "");
static_assert(bForwardLikeMatchesStd_V<const          FForwardLikeInnerType&&,                FForwardLikeOuterType& >, "");
static_assert(bForwardLikeMatchesStd_V<      volatile FForwardLikeInnerType&&,                FForwardLikeOuterType& >, "");
static_assert(bForwardLikeMatchesStd_V<const volatile FForwardLikeInnerType&&,                FForwardLikeOuterType& >, "");

static_assert(bForwardLikeMatchesStd_V<               FForwardLikeInnerType  , const          FForwardLikeOuterType& >, "");
static_assert(bForwardLikeMatchesStd_V<const          FForwardLikeInnerType  , const          FForwardLikeOuterType& >, "");
static_assert(bForwardLikeMatchesStd_V<      volatile FForwardLikeInnerType  , const          FForwardLikeOuterType& >, "");
static_assert(bForwardLikeMatchesStd_V<const volatile FForwardLikeInnerType  , const          FForwardLikeOuterType& >, "");
static_assert(bForwardLikeMatchesStd_V<               FForwardLikeInnerType& , const          FForwardLikeOuterType& >, "");
static_assert(bForwardLikeMatchesStd_V<const          FForwardLikeInnerType& , const          FForwardLikeOuterType& >, "");
static_assert(bForwardLikeMatchesStd_V<      volatile FForwardLikeInnerType& , const          FForwardLikeOuterType& >, "");
static_assert(bForwardLikeMatchesStd_V<const volatile FForwardLikeInnerType& , const          FForwardLikeOuterType& >, "");
static_assert(bForwardLikeMatchesStd_V<               FForwardLikeInnerType&&, const          FForwardLikeOuterType& >, "");
static_assert(bForwardLikeMatchesStd_V<const          FForwardLikeInnerType&&, const          FForwardLikeOuterType& >, "");
static_assert(bForwardLikeMatchesStd_V<      volatile FForwardLikeInnerType&&, const          FForwardLikeOuterType& >, "");
static_assert(bForwardLikeMatchesStd_V<const volatile FForwardLikeInnerType&&, const          FForwardLikeOuterType& >, "");

static_assert(bForwardLikeMatchesStd_V<               FForwardLikeInnerType  ,       volatile FForwardLikeOuterType& >, "");
static_assert(bForwardLikeMatchesStd_V<const          FForwardLikeInnerType  ,       volatile FForwardLikeOuterType& >, "");
static_assert(bForwardLikeMatchesStd_V<      volatile FForwardLikeInnerType  ,       volatile FForwardLikeOuterType& >, "");
static_assert(bForwardLikeMatchesStd_V<const volatile FForwardLikeInnerType  ,       volatile FForwardLikeOuterType& >, "");
static_assert(bForwardLikeMatchesStd_V<               FForwardLikeInnerType& ,       volatile FForwardLikeOuterType& >, "");
static_assert(bForwardLikeMatchesStd_V<const          FForwardLikeInnerType& ,       volatile FForwardLikeOuterType& >, "");
static_assert(bForwardLikeMatchesStd_V<      volatile FForwardLikeInnerType& ,       volatile FForwardLikeOuterType& >, "");
static_assert(bForwardLikeMatchesStd_V<const volatile FForwardLikeInnerType& ,       volatile FForwardLikeOuterType& >, "");
static_assert(bForwardLikeMatchesStd_V<               FForwardLikeInnerType&&,       volatile FForwardLikeOuterType& >, "");
static_assert(bForwardLikeMatchesStd_V<const          FForwardLikeInnerType&&,       volatile FForwardLikeOuterType& >, "");
static_assert(bForwardLikeMatchesStd_V<      volatile FForwardLikeInnerType&&,       volatile FForwardLikeOuterType& >, "");
static_assert(bForwardLikeMatchesStd_V<const volatile FForwardLikeInnerType&&,       volatile FForwardLikeOuterType& >, "");

static_assert(bForwardLikeMatchesStd_V<               FForwardLikeInnerType  ,                FForwardLikeOuterType&&>, "");
static_assert(bForwardLikeMatchesStd_V<const          FForwardLikeInnerType  ,                FForwardLikeOuterType&&>, "");
static_assert(bForwardLikeMatchesStd_V<      volatile FForwardLikeInnerType  ,                FForwardLikeOuterType&&>, "");
static_assert(bForwardLikeMatchesStd_V<const volatile FForwardLikeInnerType  ,                FForwardLikeOuterType&&>, "");
static_assert(bForwardLikeMatchesStd_V<               FForwardLikeInnerType& ,                FForwardLikeOuterType&&>, "");
static_assert(bForwardLikeMatchesStd_V<const          FForwardLikeInnerType& ,                FForwardLikeOuterType&&>, "");
static_assert(bForwardLikeMatchesStd_V<      volatile FForwardLikeInnerType& ,                FForwardLikeOuterType&&>, "");
static_assert(bForwardLikeMatchesStd_V<const volatile FForwardLikeInnerType& ,                FForwardLikeOuterType&&>, "");
static_assert(bForwardLikeMatchesStd_V<               FForwardLikeInnerType&&,                FForwardLikeOuterType&&>, "");
static_assert(bForwardLikeMatchesStd_V<const          FForwardLikeInnerType&&,                FForwardLikeOuterType&&>, "");
static_assert(bForwardLikeMatchesStd_V<      volatile FForwardLikeInnerType&&,                FForwardLikeOuterType&&>, "");
static_assert(bForwardLikeMatchesStd_V<const volatile FForwardLikeInnerType&&,                FForwardLikeOuterType&&>, "");

static_assert(bForwardLikeMatchesStd_V<               FForwardLikeInnerType  , const          FForwardLikeOuterType&&>, "");
static_assert(bForwardLikeMatchesStd_V<const          FForwardLikeInnerType  , const          FForwardLikeOuterType&&>, "");
static_assert(bForwardLikeMatchesStd_V<      volatile FForwardLikeInnerType  , const          FForwardLikeOuterType&&>, "");
static_assert(bForwardLikeMatchesStd_V<const volatile FForwardLikeInnerType  , const          FForwardLikeOuterType&&>, "");
static_assert(bForwardLikeMatchesStd_V<               FForwardLikeInnerType& , const          FForwardLikeOuterType&&>, "");
static_assert(bForwardLikeMatchesStd_V<const          FForwardLikeInnerType& , const          FForwardLikeOuterType&&>, "");
static_assert(bForwardLikeMatchesStd_V<      volatile FForwardLikeInnerType& , const          FForwardLikeOuterType&&>, "");
static_assert(bForwardLikeMatchesStd_V<const volatile FForwardLikeInnerType& , const          FForwardLikeOuterType&&>, "");
static_assert(bForwardLikeMatchesStd_V<               FForwardLikeInnerType&&, const          FForwardLikeOuterType&&>, "");
static_assert(bForwardLikeMatchesStd_V<const          FForwardLikeInnerType&&, const          FForwardLikeOuterType&&>, "");
static_assert(bForwardLikeMatchesStd_V<      volatile FForwardLikeInnerType&&, const          FForwardLikeOuterType&&>, "");
static_assert(bForwardLikeMatchesStd_V<const volatile FForwardLikeInnerType&&, const          FForwardLikeOuterType&&>, "");

static_assert(bForwardLikeMatchesStd_V<               FForwardLikeInnerType  ,       volatile FForwardLikeOuterType&&>, "");
static_assert(bForwardLikeMatchesStd_V<const          FForwardLikeInnerType  ,       volatile FForwardLikeOuterType&&>, "");
static_assert(bForwardLikeMatchesStd_V<      volatile FForwardLikeInnerType  ,       volatile FForwardLikeOuterType&&>, "");
static_assert(bForwardLikeMatchesStd_V<const volatile FForwardLikeInnerType  ,       volatile FForwardLikeOuterType&&>, "");
static_assert(bForwardLikeMatchesStd_V<               FForwardLikeInnerType& ,       volatile FForwardLikeOuterType&&>, "");
static_assert(bForwardLikeMatchesStd_V<const          FForwardLikeInnerType& ,       volatile FForwardLikeOuterType&&>, "");
static_assert(bForwardLikeMatchesStd_V<      volatile FForwardLikeInnerType& ,       volatile FForwardLikeOuterType&&>, "");
static_assert(bForwardLikeMatchesStd_V<const volatile FForwardLikeInnerType& ,       volatile FForwardLikeOuterType&&>, "");
static_assert(bForwardLikeMatchesStd_V<               FForwardLikeInnerType&&,       volatile FForwardLikeOuterType&&>, "");
static_assert(bForwardLikeMatchesStd_V<const          FForwardLikeInnerType&&,       volatile FForwardLikeOuterType&&>, "");
static_assert(bForwardLikeMatchesStd_V<      volatile FForwardLikeInnerType&&,       volatile FForwardLikeOuterType&&>, "");
static_assert(bForwardLikeMatchesStd_V<const volatile FForwardLikeInnerType&&,       volatile FForwardLikeOuterType&&>, "");

static_assert(bForwardLikeMatchesStd_V<               FForwardLikeInnerType  , const volatile FForwardLikeOuterType&&>, "");
static_assert(bForwardLikeMatchesStd_V<const          FForwardLikeInnerType  , const volatile FForwardLikeOuterType&&>, "");
static_assert(bForwardLikeMatchesStd_V<      volatile FForwardLikeInnerType  , const volatile FForwardLikeOuterType&&>, "");
static_assert(bForwardLikeMatchesStd_V<const volatile FForwardLikeInnerType  , const volatile FForwardLikeOuterType&&>, "");
static_assert(bForwardLikeMatchesStd_V<               FForwardLikeInnerType& , const volatile FForwardLikeOuterType&&>, "");
static_assert(bForwardLikeMatchesStd_V<const          FForwardLikeInnerType& , const volatile FForwardLikeOuterType&&>, "");
static_assert(bForwardLikeMatchesStd_V<      volatile FForwardLikeInnerType& , const volatile FForwardLikeOuterType&&>, "");
static_assert(bForwardLikeMatchesStd_V<const volatile FForwardLikeInnerType& , const volatile FForwardLikeOuterType&&>, "");
static_assert(bForwardLikeMatchesStd_V<const          FForwardLikeInnerType&&, const volatile FForwardLikeOuterType&&>, "");
static_assert(bForwardLikeMatchesStd_V<      volatile FForwardLikeInnerType&&, const volatile FForwardLikeOuterType&&>, "");
static_assert(bForwardLikeMatchesStd_V<const volatile FForwardLikeInnerType&&, const volatile FForwardLikeOuterType&&>, "");
