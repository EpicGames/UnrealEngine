// Copyright Epic Games, Inc. All Rights Reserved.

#include "Templates/Function.h"

namespace UE::Core::Private::Function
{
	FORCENOINLINE void CheckCallable(void* Callable)
	{
		checkf(Callable, TEXT("Attempting to call an unbound TFunction!"));
	}
}
