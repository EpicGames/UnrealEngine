// Copyright Epic Games, Inc. All Rights Reserved.

#include <type_traits>
#include "Misc/AutomationTest.h"
#include "Templates/Tuple.h"

// Pair keys have a different Get implementation, so test a pair and a non-pair

// Lvalue tuples with value elements
static_assert(std::is_same<decltype(DeclVal<               TTuple<               int      >&>().Get<0>()),                int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<               TTuple<const          int      >&>().Get<0>()), const          int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<               TTuple<      volatile int      >&>().Get<0>()),       volatile int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<               TTuple<const volatile int      >&>().Get<0>()), const volatile int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const          TTuple<               int      >&>().Get<0>()), const          int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const          TTuple<const          int      >&>().Get<0>()), const          int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const          TTuple<      volatile int      >&>().Get<0>()), const volatile int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const          TTuple<const volatile int      >&>().Get<0>()), const volatile int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<      volatile TTuple<               int      >&>().Get<0>()),       volatile int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<      volatile TTuple<const          int      >&>().Get<0>()), const volatile int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<      volatile TTuple<      volatile int      >&>().Get<0>()),       volatile int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<      volatile TTuple<const volatile int      >&>().Get<0>()), const volatile int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const volatile TTuple<               int      >&>().Get<0>()), const volatile int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const volatile TTuple<const          int      >&>().Get<0>()), const volatile int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const volatile TTuple<      volatile int      >&>().Get<0>()), const volatile int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const volatile TTuple<const volatile int      >&>().Get<0>()), const volatile int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<               TTuple<               int, char>&>().Get<0>()),                int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<               TTuple<const          int, char>&>().Get<0>()), const          int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<               TTuple<      volatile int, char>&>().Get<0>()),       volatile int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<               TTuple<const volatile int, char>&>().Get<0>()), const volatile int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const          TTuple<               int, char>&>().Get<0>()), const          int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const          TTuple<const          int, char>&>().Get<0>()), const          int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const          TTuple<      volatile int, char>&>().Get<0>()), const volatile int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const          TTuple<const volatile int, char>&>().Get<0>()), const volatile int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<      volatile TTuple<               int, char>&>().Get<0>()),       volatile int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<      volatile TTuple<const          int, char>&>().Get<0>()), const volatile int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<      volatile TTuple<      volatile int, char>&>().Get<0>()),       volatile int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<      volatile TTuple<const volatile int, char>&>().Get<0>()), const volatile int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const volatile TTuple<               int, char>&>().Get<0>()), const volatile int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const volatile TTuple<const          int, char>&>().Get<0>()), const volatile int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const volatile TTuple<      volatile int, char>&>().Get<0>()), const volatile int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const volatile TTuple<const volatile int, char>&>().Get<0>()), const volatile int&>::value, "");

// Rvalue tuples with value elements
static_assert(std::is_same<decltype(DeclVal<               TTuple<               int      >&&>().Get<0>()),                int&&>::value, "");
static_assert(std::is_same<decltype(DeclVal<               TTuple<const          int      >&&>().Get<0>()), const          int&&>::value, "");
static_assert(std::is_same<decltype(DeclVal<               TTuple<      volatile int      >&&>().Get<0>()),       volatile int&&>::value, "");
static_assert(std::is_same<decltype(DeclVal<               TTuple<const volatile int      >&&>().Get<0>()), const volatile int&&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const          TTuple<               int      >&&>().Get<0>()), const          int&&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const          TTuple<const          int      >&&>().Get<0>()), const          int&&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const          TTuple<      volatile int      >&&>().Get<0>()), const volatile int&&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const          TTuple<const volatile int      >&&>().Get<0>()), const volatile int&&>::value, "");
static_assert(std::is_same<decltype(DeclVal<      volatile TTuple<               int      >&&>().Get<0>()),       volatile int&&>::value, "");
static_assert(std::is_same<decltype(DeclVal<      volatile TTuple<const          int      >&&>().Get<0>()), const volatile int&&>::value, "");
static_assert(std::is_same<decltype(DeclVal<      volatile TTuple<      volatile int      >&&>().Get<0>()),       volatile int&&>::value, "");
static_assert(std::is_same<decltype(DeclVal<      volatile TTuple<const volatile int      >&&>().Get<0>()), const volatile int&&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const volatile TTuple<               int      >&&>().Get<0>()), const volatile int&&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const volatile TTuple<const          int      >&&>().Get<0>()), const volatile int&&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const volatile TTuple<      volatile int      >&&>().Get<0>()), const volatile int&&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const volatile TTuple<const volatile int      >&&>().Get<0>()), const volatile int&&>::value, "");
static_assert(std::is_same<decltype(DeclVal<               TTuple<               int, char>&&>().Get<0>()),                int&&>::value, "");
static_assert(std::is_same<decltype(DeclVal<               TTuple<const          int, char>&&>().Get<0>()), const          int&&>::value, "");
static_assert(std::is_same<decltype(DeclVal<               TTuple<      volatile int, char>&&>().Get<0>()),       volatile int&&>::value, "");
static_assert(std::is_same<decltype(DeclVal<               TTuple<const volatile int, char>&&>().Get<0>()), const volatile int&&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const          TTuple<               int, char>&&>().Get<0>()), const          int&&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const          TTuple<const          int, char>&&>().Get<0>()), const          int&&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const          TTuple<      volatile int, char>&&>().Get<0>()), const volatile int&&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const          TTuple<const volatile int, char>&&>().Get<0>()), const volatile int&&>::value, "");
static_assert(std::is_same<decltype(DeclVal<      volatile TTuple<               int, char>&&>().Get<0>()),       volatile int&&>::value, "");
static_assert(std::is_same<decltype(DeclVal<      volatile TTuple<const          int, char>&&>().Get<0>()), const volatile int&&>::value, "");
static_assert(std::is_same<decltype(DeclVal<      volatile TTuple<      volatile int, char>&&>().Get<0>()),       volatile int&&>::value, "");
static_assert(std::is_same<decltype(DeclVal<      volatile TTuple<const volatile int, char>&&>().Get<0>()), const volatile int&&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const volatile TTuple<               int, char>&&>().Get<0>()), const volatile int&&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const volatile TTuple<const          int, char>&&>().Get<0>()), const volatile int&&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const volatile TTuple<      volatile int, char>&&>().Get<0>()), const volatile int&&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const volatile TTuple<const volatile int, char>&&>().Get<0>()), const volatile int&&>::value, "");

// Lvalue tuples with lvalue reference elements
static_assert(std::is_same<decltype(DeclVal<               TTuple<               int&      >&>().Get<0>()),                int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<               TTuple<const          int&      >&>().Get<0>()), const          int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<               TTuple<      volatile int&      >&>().Get<0>()),       volatile int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<               TTuple<const volatile int&      >&>().Get<0>()), const volatile int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const          TTuple<               int&      >&>().Get<0>()),                int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const          TTuple<const          int&      >&>().Get<0>()), const          int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const          TTuple<      volatile int&      >&>().Get<0>()),       volatile int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const          TTuple<const volatile int&      >&>().Get<0>()), const volatile int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<      volatile TTuple<               int&      >&>().Get<0>()),                int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<      volatile TTuple<const          int&      >&>().Get<0>()), const          int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<      volatile TTuple<      volatile int&      >&>().Get<0>()),       volatile int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<      volatile TTuple<const volatile int&      >&>().Get<0>()), const volatile int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const volatile TTuple<               int&      >&>().Get<0>()),                int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const volatile TTuple<const          int&      >&>().Get<0>()), const          int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const volatile TTuple<      volatile int&      >&>().Get<0>()),       volatile int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const volatile TTuple<const volatile int&      >&>().Get<0>()), const volatile int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<               TTuple<               int&, char>&>().Get<0>()),                int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<               TTuple<const          int&, char>&>().Get<0>()), const          int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<               TTuple<      volatile int&, char>&>().Get<0>()),       volatile int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<               TTuple<const volatile int&, char>&>().Get<0>()), const volatile int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const          TTuple<               int&, char>&>().Get<0>()),                int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const          TTuple<const          int&, char>&>().Get<0>()), const          int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const          TTuple<      volatile int&, char>&>().Get<0>()),       volatile int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const          TTuple<const volatile int&, char>&>().Get<0>()), const volatile int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<      volatile TTuple<               int&, char>&>().Get<0>()),                int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<      volatile TTuple<const          int&, char>&>().Get<0>()), const          int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<      volatile TTuple<      volatile int&, char>&>().Get<0>()),       volatile int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<      volatile TTuple<const volatile int&, char>&>().Get<0>()), const volatile int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const volatile TTuple<               int&, char>&>().Get<0>()),                int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const volatile TTuple<const          int&, char>&>().Get<0>()), const          int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const volatile TTuple<      volatile int&, char>&>().Get<0>()),       volatile int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const volatile TTuple<const volatile int&, char>&>().Get<0>()), const volatile int&>::value, "");

// Rvalue tuples with lvalue reference elements
static_assert(std::is_same<decltype(DeclVal<               TTuple<               int&      >&&>().Get<0>()),                int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<               TTuple<const          int&      >&&>().Get<0>()), const          int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<               TTuple<      volatile int&      >&&>().Get<0>()),       volatile int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<               TTuple<const volatile int&      >&&>().Get<0>()), const volatile int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const          TTuple<               int&      >&&>().Get<0>()),                int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const          TTuple<const          int&      >&&>().Get<0>()), const          int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const          TTuple<      volatile int&      >&&>().Get<0>()),       volatile int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const          TTuple<const volatile int&      >&&>().Get<0>()), const volatile int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<      volatile TTuple<               int&      >&&>().Get<0>()),                int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<      volatile TTuple<const          int&      >&&>().Get<0>()), const          int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<      volatile TTuple<      volatile int&      >&&>().Get<0>()),       volatile int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<      volatile TTuple<const volatile int&      >&&>().Get<0>()), const volatile int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const volatile TTuple<               int&      >&&>().Get<0>()),                int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const volatile TTuple<const          int&      >&&>().Get<0>()), const          int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const volatile TTuple<      volatile int&      >&&>().Get<0>()),       volatile int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const volatile TTuple<const volatile int&      >&&>().Get<0>()), const volatile int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<               TTuple<               int&, char>&&>().Get<0>()),                int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<               TTuple<const          int&, char>&&>().Get<0>()), const          int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<               TTuple<      volatile int&, char>&&>().Get<0>()),       volatile int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<               TTuple<const volatile int&, char>&&>().Get<0>()), const volatile int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const          TTuple<               int&, char>&&>().Get<0>()),                int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const          TTuple<const          int&, char>&&>().Get<0>()), const          int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const          TTuple<      volatile int&, char>&&>().Get<0>()),       volatile int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const          TTuple<const volatile int&, char>&&>().Get<0>()), const volatile int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<      volatile TTuple<               int&, char>&&>().Get<0>()),                int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<      volatile TTuple<const          int&, char>&&>().Get<0>()), const          int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<      volatile TTuple<      volatile int&, char>&&>().Get<0>()),       volatile int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<      volatile TTuple<const volatile int&, char>&&>().Get<0>()), const volatile int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const volatile TTuple<               int&, char>&&>().Get<0>()),                int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const volatile TTuple<const          int&, char>&&>().Get<0>()), const          int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const volatile TTuple<      volatile int&, char>&&>().Get<0>()),       volatile int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const volatile TTuple<const volatile int&, char>&&>().Get<0>()), const volatile int&>::value, "");

// Lvalue tuples with rvalue reference elements
static_assert(std::is_same<decltype(DeclVal<               TTuple<               int&&      >&>().Get<0>()),                int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<               TTuple<const          int&&      >&>().Get<0>()), const          int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<               TTuple<      volatile int&&      >&>().Get<0>()),       volatile int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<               TTuple<const volatile int&&      >&>().Get<0>()), const volatile int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const          TTuple<               int&&      >&>().Get<0>()),                int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const          TTuple<const          int&&      >&>().Get<0>()), const          int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const          TTuple<      volatile int&&      >&>().Get<0>()),       volatile int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const          TTuple<const volatile int&&      >&>().Get<0>()), const volatile int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<      volatile TTuple<               int&&      >&>().Get<0>()),                int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<      volatile TTuple<const          int&&      >&>().Get<0>()), const          int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<      volatile TTuple<      volatile int&&      >&>().Get<0>()),       volatile int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<      volatile TTuple<const volatile int&&      >&>().Get<0>()), const volatile int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const volatile TTuple<               int&&      >&>().Get<0>()),                int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const volatile TTuple<const          int&&      >&>().Get<0>()), const          int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const volatile TTuple<      volatile int&&      >&>().Get<0>()),       volatile int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const volatile TTuple<const volatile int&&      >&>().Get<0>()), const volatile int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<               TTuple<               int&&, char>&>().Get<0>()),                int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<               TTuple<const          int&&, char>&>().Get<0>()), const          int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<               TTuple<      volatile int&&, char>&>().Get<0>()),       volatile int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<               TTuple<const volatile int&&, char>&>().Get<0>()), const volatile int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const          TTuple<               int&&, char>&>().Get<0>()),                int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const          TTuple<const          int&&, char>&>().Get<0>()), const          int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const          TTuple<      volatile int&&, char>&>().Get<0>()),       volatile int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const          TTuple<const volatile int&&, char>&>().Get<0>()), const volatile int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<      volatile TTuple<               int&&, char>&>().Get<0>()),                int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<      volatile TTuple<const          int&&, char>&>().Get<0>()), const          int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<      volatile TTuple<      volatile int&&, char>&>().Get<0>()),       volatile int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<      volatile TTuple<const volatile int&&, char>&>().Get<0>()), const volatile int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const volatile TTuple<               int&&, char>&>().Get<0>()),                int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const volatile TTuple<const          int&&, char>&>().Get<0>()), const          int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const volatile TTuple<      volatile int&&, char>&>().Get<0>()),       volatile int&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const volatile TTuple<const volatile int&&, char>&>().Get<0>()), const volatile int&>::value, "");

// Rvalue tuples with rvalue reference elements
// Note that this behavior differs from normal member access in a struct.
// An rvalue reference member of an rvalue struct used in an expression is treated as an lvalue.
// An rvalue reference member of an rvalue tuple used in an expression is treated as an rvalue.
static_assert(std::is_same<decltype(DeclVal<               TTuple<               int&&      >&&>().Get<0>()),                int&&>::value, "");
static_assert(std::is_same<decltype(DeclVal<               TTuple<const          int&&      >&&>().Get<0>()), const          int&&>::value, "");
static_assert(std::is_same<decltype(DeclVal<               TTuple<      volatile int&&      >&&>().Get<0>()),       volatile int&&>::value, "");
static_assert(std::is_same<decltype(DeclVal<               TTuple<const volatile int&&      >&&>().Get<0>()), const volatile int&&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const          TTuple<               int&&      >&&>().Get<0>()),                int&&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const          TTuple<const          int&&      >&&>().Get<0>()), const          int&&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const          TTuple<      volatile int&&      >&&>().Get<0>()),       volatile int&&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const          TTuple<const volatile int&&      >&&>().Get<0>()), const volatile int&&>::value, "");
static_assert(std::is_same<decltype(DeclVal<      volatile TTuple<               int&&      >&&>().Get<0>()),                int&&>::value, "");
static_assert(std::is_same<decltype(DeclVal<      volatile TTuple<const          int&&      >&&>().Get<0>()), const          int&&>::value, "");
static_assert(std::is_same<decltype(DeclVal<      volatile TTuple<      volatile int&&      >&&>().Get<0>()),       volatile int&&>::value, "");
static_assert(std::is_same<decltype(DeclVal<      volatile TTuple<const volatile int&&      >&&>().Get<0>()), const volatile int&&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const volatile TTuple<               int&&      >&&>().Get<0>()),                int&&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const volatile TTuple<const          int&&      >&&>().Get<0>()), const          int&&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const volatile TTuple<      volatile int&&      >&&>().Get<0>()),       volatile int&&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const volatile TTuple<const volatile int&&      >&&>().Get<0>()), const volatile int&&>::value, "");
static_assert(std::is_same<decltype(DeclVal<               TTuple<               int&&, char>&&>().Get<0>()),                int&&>::value, "");
static_assert(std::is_same<decltype(DeclVal<               TTuple<const          int&&, char>&&>().Get<0>()), const          int&&>::value, "");
static_assert(std::is_same<decltype(DeclVal<               TTuple<      volatile int&&, char>&&>().Get<0>()),       volatile int&&>::value, "");
static_assert(std::is_same<decltype(DeclVal<               TTuple<const volatile int&&, char>&&>().Get<0>()), const volatile int&&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const          TTuple<               int&&, char>&&>().Get<0>()),                int&&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const          TTuple<const          int&&, char>&&>().Get<0>()), const          int&&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const          TTuple<      volatile int&&, char>&&>().Get<0>()),       volatile int&&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const          TTuple<const volatile int&&, char>&&>().Get<0>()), const volatile int&&>::value, "");
static_assert(std::is_same<decltype(DeclVal<      volatile TTuple<               int&&, char>&&>().Get<0>()),                int&&>::value, "");
static_assert(std::is_same<decltype(DeclVal<      volatile TTuple<const          int&&, char>&&>().Get<0>()), const          int&&>::value, "");
static_assert(std::is_same<decltype(DeclVal<      volatile TTuple<      volatile int&&, char>&&>().Get<0>()),       volatile int&&>::value, "");
static_assert(std::is_same<decltype(DeclVal<      volatile TTuple<const volatile int&&, char>&&>().Get<0>()), const volatile int&&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const volatile TTuple<               int&&, char>&&>().Get<0>()),                int&&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const volatile TTuple<const          int&&, char>&&>().Get<0>()), const          int&&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const volatile TTuple<      volatile int&&, char>&&>().Get<0>()),       volatile int&&>::value, "");
static_assert(std::is_same<decltype(DeclVal<const volatile TTuple<const volatile int&&, char>&&>().Get<0>()), const volatile int&&>::value, "");

// Check that TTupleElement works for values, lvalue references and rvalue references, even with qualifiers on the tuple
static_assert(std::is_same<TTupleElement<0,                TTuple<double, float&, char&&>>::Type, double>::value, "");
static_assert(std::is_same<TTupleElement<1,                TTuple<double, float&, char&&>>::Type, float&>::value, "");
static_assert(std::is_same<TTupleElement<2,                TTuple<double, float&, char&&>>::Type, char&&>::value, "");
static_assert(std::is_same<TTupleElement<0, const          TTuple<double, float&, char&&>>::Type, double>::value, "");
static_assert(std::is_same<TTupleElement<1, const          TTuple<double, float&, char&&>>::Type, float&>::value, "");
static_assert(std::is_same<TTupleElement<2, const          TTuple<double, float&, char&&>>::Type, char&&>::value, "");
static_assert(std::is_same<TTupleElement<0,       volatile TTuple<double, float&, char&&>>::Type, double>::value, "");
static_assert(std::is_same<TTupleElement<1,       volatile TTuple<double, float&, char&&>>::Type, float&>::value, "");
static_assert(std::is_same<TTupleElement<2,       volatile TTuple<double, float&, char&&>>::Type, char&&>::value, "");
static_assert(std::is_same<TTupleElement<0, const volatile TTuple<double, float&, char&&>>::Type, double>::value, "");
static_assert(std::is_same<TTupleElement<1, const volatile TTuple<double, float&, char&&>>::Type, float&>::value, "");
static_assert(std::is_same<TTupleElement<2, const volatile TTuple<double, float&, char&&>>::Type, char&&>::value, "");

// Check that TTupleIndex works for values, lvalue references and rvalue references, even with qualifiers on the tuple
static_assert(TTupleIndex<double,                TTuple<double, float&, char&&>>::Value == 0, "");
static_assert(TTupleIndex<float&,                TTuple<double, float&, char&&>>::Value == 1, "");
static_assert(TTupleIndex<char&&,                TTuple<double, float&, char&&>>::Value == 2, "");
static_assert(TTupleIndex<double, const          TTuple<double, float&, char&&>>::Value == 0, "");
static_assert(TTupleIndex<float&, const          TTuple<double, float&, char&&>>::Value == 1, "");
static_assert(TTupleIndex<char&&, const          TTuple<double, float&, char&&>>::Value == 2, "");
static_assert(TTupleIndex<double,       volatile TTuple<double, float&, char&&>>::Value == 0, "");
static_assert(TTupleIndex<float&,       volatile TTuple<double, float&, char&&>>::Value == 1, "");
static_assert(TTupleIndex<char&&,       volatile TTuple<double, float&, char&&>>::Value == 2, "");
static_assert(TTupleIndex<double, const volatile TTuple<double, float&, char&&>>::Value == 0, "");
static_assert(TTupleIndex<float&, const volatile TTuple<double, float&, char&&>>::Value == 1, "");
static_assert(TTupleIndex<char&&, const volatile TTuple<double, float&, char&&>>::Value == 2, "");

// Check that TIsTuple works for values, lvalue references and rvalue references, even with qualifiers on the tuple
static_assert(!TIsTuple<double>::Value										, "");
static_assert(TIsTuple<               TTuple<double, float&, char&&>>::Value, "");
static_assert(TIsTuple<const          TTuple<double, float&, char&&>>::Value, "");
static_assert(TIsTuple<      volatile TTuple<double, float&, char&&>>::Value, "");
static_assert(TIsTuple<const volatile TTuple<double, float&, char&&>>::Value, "");

using TestPopTupleType = TTuple<float, char, int, void, bool>;

// Check that TTuplePopBack works
static_assert(std::is_same_v<TTuplePopBack<TestPopTupleType, 0>::Type, TTuple<float, char, int, void, bool>>);
static_assert(std::is_same_v<TTuplePopBack<TestPopTupleType, 1>::Type, TTuple<float, char, int, void>>);
static_assert(std::is_same_v<TTuplePopBack<TestPopTupleType, 2>::Type, TTuple<float, char, int>>);
static_assert(std::is_same_v<TTuplePopBack<TestPopTupleType, 3>::Type, TTuple<float, char>>);
static_assert(std::is_same_v<TTuplePopBack<TestPopTupleType, 4>::Type, TTuple<float>>);
static_assert(std::is_same_v<TTuplePopBack<TestPopTupleType, 5>::Type, TTuple<>>);

// Check that TTuplePopFront works
static_assert(std::is_same_v<TTuplePopFront<TestPopTupleType, 0>::Type, TTuple<float, char, int, void, bool>>);
static_assert(std::is_same_v<TTuplePopFront<TestPopTupleType, 1>::Type, TTuple<char, int, void, bool>>);
static_assert(std::is_same_v<TTuplePopFront<TestPopTupleType, 2>::Type, TTuple<int, void, bool>>);
static_assert(std::is_same_v<TTuplePopFront<TestPopTupleType, 3>::Type, TTuple<void, bool>>);
static_assert(std::is_same_v<TTuplePopFront<TestPopTupleType, 4>::Type, TTuple<bool>>);
static_assert(std::is_same_v<TTuplePopFront<TestPopTupleType, 5>::Type, TTuple<>>);

// Check that TTuplePopBack preserves qualifiers
static_assert(std::is_same_v<TTuplePopBack<const          TestPopTupleType, 2>::Type, const          TTuple<float, char, int>>);
static_assert(std::is_same_v<TTuplePopBack<      volatile TestPopTupleType, 2>::Type,       volatile TTuple<float, char, int>>);
static_assert(std::is_same_v<TTuplePopBack<const volatile TestPopTupleType, 2>::Type, const volatile TTuple<float, char, int>>);

// Check that TTuplePopFront preserves qualifiers
static_assert(std::is_same_v<TTuplePopFront<const          TestPopTupleType, 2>::Type, const          TTuple<int, void, bool>>);
static_assert(std::is_same_v<TTuplePopFront<      volatile TestPopTupleType, 2>::Type,       volatile TTuple<int, void, bool>>);
static_assert(std::is_same_v<TTuplePopFront<const volatile TestPopTupleType, 2>::Type, const volatile TTuple<int, void, bool>>);

// These shouldn't compile - ideally giving a meaningful error message
#if 0
	// TTupleElement passed a non-tuple
	static_assert(std::is_same<TTupleElement<0, int>::Type, double>::value, "");

	// TTupleIndex passed a non-tuple
	static_assert(TTupleIndex<int, int>::Value == 0, "");

	// Invalid index
	static_assert(std::is_same<TTupleElement<4, TTuple<double, float&, char&&>>::Type, double>::value, "");

	// Type not in tuple
	static_assert(TTupleIndex<int, TTuple<double, float&, char&&>>::Value == 0, "");

	// Type appears multiple times in tuple
	static_assert(TTupleIndex<int, TTuple<int, float&, int>>::Value == 0, "");

	// Multiple arguments
	static_assert(TIsTuple<double, double>::Value, "");
#endif

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FTuplesTest, "System.Core.Tuples.Smoke", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool FTuplesTest::RunTest(const FString& Parameters)
{
	// Test getting references from tuples - this catches an issue found when MSVC returns the wrong address of a reference-to
	// array element in an rvalue tuple.  See UE_TUPLE_REFERENCE_WORKAROUND.
	{
		auto TestTupleValuesImpl = [](auto&& TupleRef, const auto& Addresses)
		{
			VisitTupleElements(
				[](const auto& Element, const void* Address)
				{
					check(&Forward<decltype(Element)>(Element) == Address);
				},
				Forward<decltype(TupleRef)>(TupleRef),
				Addresses
			);
		};

		auto TestTupleValues = [&](auto&& TupleRef, const auto& Addresses)
		{
			using TupleRefType = decltype(TupleRef);

			// Test as an lvalue tuple and as an rvalue tuple
			TestTupleValuesImpl(TupleRef, Addresses);
			TestTupleValuesImpl(Forward<decltype(TupleRef)>(TupleRef), Addresses);
		};

		int Int = 5;
		float Float = 3.14f;
		char Char = 'x';
		int Array[] = { 1, 2, 3 };
		auto Addresses = MakeTuple(&Int, &Float, &Char, &Array[0]);
		TestTupleValues(TTuple<const int&, const float&, const char&, const int(&)[3]>(Int, Float, Char, Array), Addresses);
		TestTupleValues(TTuple<int&&, float&&, char&&, const int(&&)[3]>(MoveTemp(Int), MoveTemp(Float), MoveTemp(Char), MoveTemp(Array)), Addresses);

		// Test a pair separately as they have specializations
		auto AddressesPair = MakeTuple(&Float, &Array[0]);
		TestTupleValues(TTuple<const float&, const int(&)[3]>(Float, Array), AddressesPair);
		TestTupleValues(TTuple<float&&, const int(&&)[3]>( MoveTemp(Float), MoveTemp(Array)), AddressesPair);
	}
	return true;
}
