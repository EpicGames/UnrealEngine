// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/CheckScope.h"
#include "Containers/StringConv.h"
#include "CoreGlobals.h"
#include "Misc/OutputDeviceError.h"

class FCheckScopeOutputDeviceError : public FOutputDeviceError
{
public:
	FCheckScopeOutputDeviceError(FOutputDeviceError* Error, const ANSICHAR* Msg)
		: DeviceError(Error)
		, ExpectedMsg(Msg)
		, Count(0)
	{
	}

	virtual void HandleError() override
	{
		if (DeviceError)
		{
			DeviceError->HandleError();
		}
	}

	void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const FName& Category) override
	{
		if (ExpectedMsg)
		{
			auto Str = StringCast<TCHAR>(ExpectedMsg);
			auto Found = FPlatformString::Strstr(V, Str.Get());
			if (Found)
			{
				++Count;
			}
		}
		else
		{
			++Count;
		}
	}

	void SetDeviceError(FOutputDeviceError* Error)
	{
		DeviceError = Error;
	}

	FOutputDeviceError* GetDeviceError() const
	{
		return DeviceError;
	}

	int GetCount() { return Count; }

private:
	FOutputDeviceError* DeviceError;
	const ANSICHAR* ExpectedMsg;
	int Count;
};

FCheckScope::FCheckScope(const ANSICHAR* Msg)
	: DeviceError(new FCheckScopeOutputDeviceError(GError, Msg))
#if !UE_BUILD_SHIPPING
	, bIgnoreDebugger(GIgnoreDebugger)
#else
	, bIgnoreDebugger(false)
#endif
	, bCriticalError(GIsCriticalError)
{
#if !UE_BUILD_SHIPPING
	GIgnoreDebugger = true;
#endif
	GError = DeviceError;
	GIsCriticalError = true; //set to true to disable printf of error message which causes Horde to flag the lines as an error
}

FCheckScope::FCheckScope()
	: FCheckScope(nullptr)	
{
}

FCheckScope::~FCheckScope()
{
#if !UE_BUILD_SHIPPING
	GIgnoreDebugger = bIgnoreDebugger;
#endif
	GError = DeviceError->GetDeviceError();
	GIsCriticalError = bCriticalError;
	delete DeviceError;
}

int
FCheckScope::GetCount()
{
	return DeviceError->GetCount();
}