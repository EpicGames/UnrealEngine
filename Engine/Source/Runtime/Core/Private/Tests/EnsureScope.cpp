// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/EnsureScope.h"
#include "Containers/StringConv.h"

FEnsureScope::FEnsureScope()
	: FEnsureScope([](const FEnsureHandlerArgs&) { return true; })
{
}


FEnsureScope::FEnsureScope(const ANSICHAR* ExpectedMsg)
	: FEnsureScope([ExpectedMsg](const FEnsureHandlerArgs& Args)
		{
			return FPlatformString::Stricmp(ExpectedMsg, Args.Message) == 0;
		})
{
}

FEnsureScope::FEnsureScope(TFunction<bool(const FEnsureHandlerArgs& Args)> EnsureFunc)
	: Count(0)
{
#if DO_ENSURE
	OldHandler = SetEnsureHandler([this, EnsureFunc](const FEnsureHandlerArgs& Args) -> bool
		{
			bool Handled = EnsureFunc(Args);
			if (Handled)
			{
				++Count;
			}
			return Handled;
		});
#endif
}

FEnsureScope::~FEnsureScope()
{
#if DO_ENSURE
	SetEnsureHandler(OldHandler);
#endif
}