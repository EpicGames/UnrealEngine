// Copyright Epic Games, Inc. All Rights Reserved.

#include "HAL/FileManager.h"

#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FFileManagerReaderTest, "System.Core.HAL.FileManager.Reader", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)

bool FFileManagerReaderTest::RunTest(const FString& Parameters)
{
	return true;
}

#endif