// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "GenericPlatform/GenericPlatformMath.h"
#include "Math/UnrealMathUtility.h"
#include "Templates/UnrealTemplate.h"
#include "Containers/UnrealString.h"
#include "UObject/NameTypes.h"
#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

struct TestA
{
	virtual ~TestA() {}
	virtual void TestAA() 
	{
		Space[0] = 1;
	}
	uint8 Space[64];
};


struct TestB
{
	virtual ~TestB() {}
	virtual void TestBB() 
	{
		Space[5] = 1;
	}
	uint8 Space[96];
};


struct TestC : public TestA, TestB
{
	int i;
};

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FPlatformVerificationTest, "System.Core.HAL.Platform Verification", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::SmokeFilter)

bool FPlatformVerificationTest::RunTest (const FString& Parameters)
{
	PTRINT Offset1 = VTABLE_OFFSET(TestC, TestB);
	PTRINT Offset2 = VTABLE_OFFSET(TestC, TestA);
	check(Offset1 == 64 + sizeof(void*));
	check(Offset2 == 0);
	int32 Test = 0x12345678;
#if PLATFORM_LITTLE_ENDIAN
	check(*(uint8*)&Test == 0x78);
#else
	check(*(uint8*)&Test == 0x12);
#endif

	FPlatformMath::AutoTest();

#if WITH_EDITORONLY_DATA
	check(FPlatformProperties::HasEditorOnlyData());
#else
	check(!FPlatformProperties::HasEditorOnlyData());
#endif

	check(FPlatformProperties::HasEditorOnlyData() != FPlatformProperties::RequiresCookedData());

#if PLATFORM_LITTLE_ENDIAN
	check(FPlatformProperties::IsLittleEndian());
#else
	check(!FPlatformProperties::IsLittleEndian());
#endif
	check(FPlatformProperties::PlatformName());

	check(FString(FPlatformProperties::PlatformName()).Len() > 0); 

	static_assert(alignof(int32) == 4, "Align of int32 is not 4."); //Hmmm, this would be very strange, ok maybe, but strange

	MS_ALIGN(16) struct FTestAlign
	{
		uint8 Test;
	} GCC_ALIGN(16);

	static_assert(alignof(FTestAlign) == 16, "Align of FTestAlign is not 16.");

	return true;
}

#endif //WITH_DEV_AUTOMATION_TESTS
