// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/CompactBinary.h"

#include "IO/IoHash.h"
#include "Memory/CompositeBuffer.h"
#include "Misc/AutomationTest.h"
#include "Misc/DateTime.h"
#include "Misc/Guid.h"
#include "Misc/Timespan.h"
#include "Serialization/CompactBinaryValidation.h"
#include "Serialization/CompactBinaryWriter.h"
#include "Serialization/VarInt.h"

#if WITH_DEV_AUTOMATION_TESTS

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

static constexpr EAutomationTestFlags CompactBinaryTestFlags =
	EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter;

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

template <ECbFieldType FieldType>
struct TCbFieldTypeAccessors;

template <ECbFieldType FieldType>
using TCbFieldValueType = typename TCbFieldTypeAccessors<FCbFieldType::GetType(FieldType)>::ValueType;

#define UE_CBFIELD_TYPE_ACCESSOR_NO_DEFAULT(FieldType, InIsTypeFn, InAsTypeFn, InValueType)                           \
	template <>                                                                                                       \
	struct TCbFieldTypeAccessors<ECbFieldType::FieldType>                                                             \
	{                                                                                                                 \
		using ValueType = InValueType;                                                                                \
		static constexpr bool (FCbFieldView::*IsType)() const = &FCbFieldView::InIsTypeFn;                                    \
		static auto AsType(FCbFieldView& Field, ValueType) { return Field.InAsTypeFn(); }                                 \
	};

#define UE_CBFIELD_TYPE_ACCESSOR_EX(FieldType, InIsTypeFn, InAsTypeFn, InValueType, InDefaultType)                    \
	template <>                                                                                                       \
	struct TCbFieldTypeAccessors<ECbFieldType::FieldType>                                                             \
	{                                                                                                                 \
		using ValueType = InValueType;                                                                                \
		static constexpr bool (FCbFieldView::*IsType)() const = &FCbFieldView::InIsTypeFn;                                    \
		static constexpr InValueType (FCbFieldView::*AsType)(InDefaultType) = &FCbFieldView::InAsTypeFn;                      \
	};

#define UE_CBFIELD_TYPE_ACCESSOR(FieldType, InIsTypeFn, InAsTypeFn, InValueType)                                      \
	UE_CBFIELD_TYPE_ACCESSOR_EX(FieldType, InIsTypeFn, InAsTypeFn, InValueType, InValueType)

UE_CBFIELD_TYPE_ACCESSOR_NO_DEFAULT(Object, IsObject, AsObjectView, FCbObjectView);
UE_CBFIELD_TYPE_ACCESSOR_NO_DEFAULT(UniformObject, IsObject, AsObjectView, FCbObjectView);
UE_CBFIELD_TYPE_ACCESSOR_NO_DEFAULT(Array, IsArray, AsArrayView, FCbArrayView);
UE_CBFIELD_TYPE_ACCESSOR_NO_DEFAULT(UniformArray, IsArray, AsArrayView, FCbArrayView);
UE_CBFIELD_TYPE_ACCESSOR(Binary, IsBinary, AsBinaryView, FMemoryView);
UE_CBFIELD_TYPE_ACCESSOR(String, IsString, AsString, FUtf8StringView);
UE_CBFIELD_TYPE_ACCESSOR(IntegerPositive, IsInteger, AsUInt64, uint64);
UE_CBFIELD_TYPE_ACCESSOR(IntegerNegative, IsInteger, AsInt64, int64);
UE_CBFIELD_TYPE_ACCESSOR(Float32, IsFloat, AsFloat, float);
UE_CBFIELD_TYPE_ACCESSOR(Float64, IsFloat, AsDouble, double);
UE_CBFIELD_TYPE_ACCESSOR(BoolFalse, IsBool, AsBool, bool);
UE_CBFIELD_TYPE_ACCESSOR(BoolTrue, IsBool, AsBool, bool);
UE_CBFIELD_TYPE_ACCESSOR_EX(ObjectAttachment, IsObjectAttachment, AsObjectAttachment, FIoHash, const FIoHash&);
UE_CBFIELD_TYPE_ACCESSOR_EX(BinaryAttachment, IsBinaryAttachment, AsBinaryAttachment, FIoHash, const FIoHash&);
UE_CBFIELD_TYPE_ACCESSOR_EX(Hash, IsHash, AsHash, FIoHash, const FIoHash&);
UE_CBFIELD_TYPE_ACCESSOR_EX(Uuid, IsUuid, AsUuid, FGuid, const FGuid&);
UE_CBFIELD_TYPE_ACCESSOR(DateTime, IsDateTime, AsDateTimeTicks, int64);
UE_CBFIELD_TYPE_ACCESSOR(TimeSpan, IsTimeSpan, AsTimeSpanTicks, int64);
UE_CBFIELD_TYPE_ACCESSOR_EX(ObjectId, IsObjectId, AsObjectId, FCbObjectId, const FCbObjectId&);
UE_CBFIELD_TYPE_ACCESSOR(CustomById, IsCustomById, AsCustomById, FCbCustomById);
UE_CBFIELD_TYPE_ACCESSOR(CustomByName, IsCustomByName, AsCustomByName, FCbCustomByName);

struct FCbFieldObjectAccessors
{
	static constexpr bool (FCbFieldView::*IsType)() const = &FCbFieldView::IsObject;
	static auto AsType(FCbField& Field, const FCbObject&) { return Field.AsObject(); }
};

struct FCbFieldArrayAccessors
{
	static constexpr bool (FCbFieldView::*IsType)() const = &FCbFieldView::IsArray;
	static auto AsType(FCbField& Field, const FCbArray&) { return Field.AsArray(); }
};

struct FCbAttachmentAccessors
{
	static constexpr bool (FCbFieldView::*IsType)() const = &FCbFieldView::IsAttachment;
	static constexpr FIoHash (FCbFieldView::*AsType)(const FIoHash&) = &FCbFieldView::AsAttachment;
};

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

class FCbFieldTestBase : public FAutomationTestBase
{
protected:
	using FAutomationTestBase::FAutomationTestBase;
	using FAutomationTestBase::TestEqual;

	bool TestEqual(const FString& What, const FCbArrayView& Actual, const FCbArrayView& Expected)
	{
		return TestTrue(What, Actual.Equals(Expected));
	}

	bool TestEqual(const FString& What, const FCbArray& Actual, const FCbArray& Expected)
	{
		return TestTrue(What, Actual.Equals(Expected));
	}

	bool TestEqual(const FString& What, const FCbObjectView& Actual, const FCbObjectView& Expected)
	{
		return TestTrue(What, Actual.Equals(Expected));
	}

	bool TestEqual(const FString& What, const FCbObject& Actual, const FCbObject& Expected)
	{
		return TestTrue(What, Actual.Equals(Expected));
	}

	bool TestEqual(const FString& What, const FCbCustomById& Actual, const FCbCustomById& Expected)
	{
		return TestTrue(What, Actual.Id == Expected.Id && Actual.Data == Expected.Data);
	}

	bool TestEqual(const FString& What, const FCbCustomByName& Actual, const FCbCustomByName& Expected)
	{
		return TestTrue(What, Actual.Name.Equals(Expected.Name, ESearchCase::CaseSensitive) && Actual.Data == Expected.Data);
	}

	bool TestEqualBytes(const TCHAR* What, FMemoryView Actual, TArrayView<const uint8> Expected)
	{
		return TestTrue(What, Actual.EqualBytes(MakeMemoryView(Expected)));
	}

	template <ECbFieldType FieldType, typename T = TCbFieldValueType<FieldType>, typename AccessorsType = TCbFieldTypeAccessors<FieldType>, typename FieldArgType = FCbFieldView>
	void TestFieldNoClone(const TCHAR* What, FieldArgType& Field, T ExpectedValue = T(), T DefaultValue = T(), ECbFieldError ExpectedError = ECbFieldError::None, const AccessorsType& Accessors = AccessorsType())
	{
		TestEqual(FString::Printf(TEXT("FCbFieldView::Is[Type](%s)"), What), Invoke(Accessors.IsType, Field), ExpectedError != ECbFieldError::TypeError);
		if (ExpectedError == ECbFieldError::None && !Field.IsBool())
		{
			TestFalse(FString::Printf(TEXT("FCbFieldView::AsBool(%s) == false"), What), Field.AsBool());
			TestTrue(FString::Printf(TEXT("FCbFieldView::AsBool(%s) -> HasError()"), What), Field.HasError());
			TestEqual(FString::Printf(TEXT("FCbFieldView::AsBool(%s) -> GetError() == TypeError"), What), Field.GetError(), ECbFieldError::TypeError);
		}
		TestEqual(FString::Printf(TEXT("FCbFieldView::As[Type](%s) -> Equal"), What), Invoke(Accessors.AsType, Field, DefaultValue), ExpectedValue);
		TestEqual(FString::Printf(TEXT("FCbFieldView::As[Type](%s) -> HasError()"), What), Field.HasError(), ExpectedError != ECbFieldError::None);
		TestEqual(FString::Printf(TEXT("FCbFieldView::As[Type](%s) -> GetError()"), What), Field.GetError(), ExpectedError);
	}

	template <ECbFieldType FieldType, typename T = TCbFieldValueType<FieldType>, typename AccessorsType = TCbFieldTypeAccessors<FieldType>>
	void TestFieldBytesNoClone(const TCHAR* What, TArrayView<const uint8> Value, T ExpectedValue = T(), T DefaultValue = T(), ECbFieldError ExpectedError = ECbFieldError::None, const AccessorsType& Accessors = AccessorsType())
	{
		FCbFieldView Field(Value.GetData(), FieldType);
		TestFieldNoClone<FieldType>(What, Field, ExpectedValue, DefaultValue, ExpectedError, Accessors);
	}

	template <ECbFieldType FieldType, typename T = TCbFieldValueType<FieldType>, typename AccessorsType = TCbFieldTypeAccessors<FieldType>>
	void TestField(const TCHAR* What, FCbFieldView& Field, T ExpectedValue = T(), T DefaultValue = T(), ECbFieldError ExpectedError = ECbFieldError::None, const AccessorsType& Accessors = AccessorsType())
	{
		TestFieldNoClone<FieldType>(What, Field, ExpectedValue, DefaultValue, ExpectedError, Accessors);
		FCbField FieldClone = FCbField::Clone(Field);
		TestFieldNoClone<FieldType>(*FString::Printf(TEXT("%s, Clone"), What), FieldClone, ExpectedValue, DefaultValue, ExpectedError, Accessors);
		TestTrue(FString::Printf(TEXT("FCbFieldView::Equals(%s)"), What), Field.Equals(FieldClone));
	}

	template <ECbFieldType FieldType, typename T = TCbFieldValueType<FieldType>, typename AccessorsType = TCbFieldTypeAccessors<FieldType>>
	void TestFieldBytes(const TCHAR* What, TArrayView<const uint8> Value, T ExpectedValue = T(), T DefaultValue = T(), ECbFieldError ExpectedError = ECbFieldError::None, const AccessorsType& Accessors = AccessorsType())
	{
		FCbFieldView Field(Value.GetData(), FieldType);
		TestEqual(FString::Printf(TEXT("FCbFieldView::GetSize(%s)"), What), Field.GetSize(), uint64(Value.Num()) + uint64(FCbFieldType::HasFieldType(FieldType) ? 0 : 1));
		TestTrue(FString::Printf(TEXT("FCbFieldView::HasValue(%s)"), What), Field.HasValue());
		TestFalse(FString::Printf(TEXT("FCbFieldView::HasError(%s) == false"), What), Field.HasError());
		TestEqual(FString::Printf(TEXT("FCbFieldView::GetError(%s) == None"), What), Field.GetError(), ECbFieldError::None);
		TestField<FieldType>(What, Field, ExpectedValue, DefaultValue, ExpectedError, Accessors);
	}

	template <ECbFieldType FieldType, typename T = TCbFieldValueType<FieldType>, typename AccessorsType = TCbFieldTypeAccessors<FieldType>, typename FieldArgType = FCbFieldView>
	void TestFieldError(const TCHAR* What, FieldArgType& Field, ECbFieldError ExpectedError, T ExpectedValue = T(), const AccessorsType& Accessors = AccessorsType())
	{
		TestFieldNoClone<FieldType>(What, Field, ExpectedValue, ExpectedValue, ExpectedError, Accessors);
	}

	template <ECbFieldType FieldType, typename T = TCbFieldValueType<FieldType>, typename AccessorsType = TCbFieldTypeAccessors<FieldType>>
	void TestFieldBytesError(const TCHAR* What, TArrayView<const uint8> Value, ECbFieldError ExpectedError, T ExpectedValue = T(), const AccessorsType& Accessors = AccessorsType())
	{
		FCbFieldView Field(Value.GetData(), FieldType);
		TestFieldError<FieldType>(What, Field, ExpectedError, ExpectedValue, Accessors);
	}
};

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FCbFieldNoneTest, FCbFieldTestBase, "System.Core.Serialization.CbField.None", CompactBinaryTestFlags)
bool FCbFieldNoneTest::RunTest(const FString& Parameters)
{
	// Test FCbFieldView()
	{
		constexpr FCbFieldView DefaultField;
		static_assert(!DefaultField.HasName(), "Error in HasName()");
		static_assert(!DefaultField.HasValue(), "Error in HasValue()");
		static_assert(!DefaultField.HasError(), "Error in HasError()");
		static_assert(DefaultField.GetError() == ECbFieldError::None, "Error in GetError()");
		TestEqual(TEXT("FCbFieldView()::GetSize() == 1"), DefaultField.GetSize(), uint64(1));
		TestEqual(TEXT("FCbFieldView()::GetName().Len() == 0"), DefaultField.GetName().Len(), 0);
		TestFalse(TEXT("!FCbFieldView()::HasName()"), DefaultField.HasName());
		TestFalse(TEXT("!FCbFieldView()::HasValue()"), DefaultField.HasValue());
		TestFalse(TEXT("!FCbFieldView()::HasError()"), DefaultField.HasError());
		TestEqual(TEXT("FCbFieldView()::GetError() == None"), DefaultField.GetError(), ECbFieldError::None);
		TestEqual(TEXT("FCbFieldView()::GetHash()"), DefaultField.GetHash(), FIoHash::HashBuffer(MakeMemoryView<uint8>({uint8(ECbFieldType::None)})));
		FMemoryView View;
		TestFalse(TEXT("FCbFieldView()::TryGetView()"), DefaultField.TryGetView(View));
	}

	// Test FCbFieldView(None)
	{
		FCbFieldView NoneField(nullptr, ECbFieldType::None);
		TestEqual(TEXT("FCbFieldView(None)::GetSize() == 1"), NoneField.GetSize(), uint64(1));
		TestEqual(TEXT("FCbFieldView(None)::GetName().Len() == 0"), NoneField.GetName().Len(), 0);
		TestFalse(TEXT("!FCbFieldView(None)::HasName()"), NoneField.HasName());
		TestFalse(TEXT("!FCbFieldView(None)::HasValue()"), NoneField.HasValue());
		TestFalse(TEXT("!FCbFieldView(None)::HasError()"), NoneField.HasError());
		TestEqual(TEXT("FCbFieldView(None)::GetError() == None"), NoneField.GetError(), ECbFieldError::None);
		TestEqual(TEXT("FCbFieldView(None)::GetHash()"), NoneField.GetHash(), FCbFieldView().GetHash());
		FMemoryView View;
		TestFalse(TEXT("FCbFieldView(None)::TryGetView()"), NoneField.TryGetView(View));
	}

	// Test FCbFieldView(None|Type|Name)
	{
		constexpr ECbFieldType FieldType = ECbFieldType::None | ECbFieldType::HasFieldName;
		constexpr const ANSICHAR NoneBytes[] = { ANSICHAR(FieldType), 4, 'N', 'a', 'm', 'e' };
		FCbFieldView NoneField(NoneBytes);
		TestEqual(TEXT("FCbFieldView(None|Type|Name)::GetSize()"), NoneField.GetSize(), uint64(sizeof(NoneBytes)));
		TestEqual(TEXT("FCbFieldView(None|Type|Name)::GetName()"), NoneField.GetName(), UTF8TEXTVIEW("Name"));
		TestTrue(TEXT("FCbFieldView(None|Type|Name)::HasName()"), NoneField.HasName());
		TestFalse(TEXT("!FCbFieldView(None|Type|Name)::HasValue()"), NoneField.HasValue());
		TestEqual(TEXT("FCbFieldView(None|Type|Name)::GetHash()"), NoneField.GetHash(), FIoHash::HashBuffer(MakeMemoryView(NoneBytes)));
		FMemoryView View;
		TestTrue(TEXT("FCbFieldView(None|Type|Name)::TryGetView()"), NoneField.TryGetView(View) && View == MakeMemoryView(NoneBytes));

		uint8 CopyBytes[sizeof(NoneBytes)];
		NoneField.CopyTo(MakeMemoryView(CopyBytes));
		TestTrue(TEXT("FCbFieldView(None|Type|Name)::CopyTo()"), MakeMemoryView(NoneBytes).EqualBytes(MakeMemoryView(CopyBytes)));
	}

	// Test FCbFieldView(None|Type)
	{
		constexpr ECbFieldType FieldType = ECbFieldType::None;
		constexpr const ANSICHAR NoneBytes[] = { ANSICHAR(FieldType) };
		FCbFieldView NoneField(NoneBytes);
		TestEqual(TEXT("FCbFieldView(None|Type)::GetSize()"), NoneField.GetSize(), uint64(sizeof(NoneBytes)));
		TestEqual(TEXT("FCbFieldView(None|Type)::GetName()"), NoneField.GetName().Len(), 0);
		TestFalse(TEXT("FCbFieldView(None|Type)::HasName()"), NoneField.HasName());
		TestFalse(TEXT("!FCbFieldView(None|Type)::HasValue()"), NoneField.HasValue());
		TestEqual(TEXT("FCbFieldView(None|Type)::GetHash()"), NoneField.GetHash(), FCbFieldView().GetHash());
		FMemoryView View;
		TestTrue(TEXT("FCbFieldView(None|Type)::TryGetView()"), NoneField.TryGetView(View) && View == MakeMemoryView(NoneBytes));
	}

	// Test FCbFieldView(None|Name)
	{
		constexpr ECbFieldType FieldType = ECbFieldType::None | ECbFieldType::HasFieldName;
		constexpr const ANSICHAR NoneBytes[] = { ANSICHAR(FieldType), 4, 'N', 'a', 'm', 'e' };
		FCbFieldView NoneField(NoneBytes + 1, FieldType);
		TestEqual(TEXT("FCbFieldView(None|Name)::GetSize()"), NoneField.GetSize(), uint64(sizeof(NoneBytes)));
		TestEqual(TEXT("FCbFieldView(None|Name)::GetName()"), NoneField.GetName(), UTF8TEXTVIEW("Name"));
		TestTrue(TEXT("FCbFieldView(None|Name)::HasName()"), NoneField.HasName());
		TestFalse(TEXT("!FCbFieldView(None|Name)::HasValue()"), NoneField.HasValue());
		TestEqual(TEXT("FCbFieldView(None|Name)::GetHash()"), NoneField.GetHash(), FIoHash::HashBuffer(MakeMemoryView(NoneBytes)));
		FMemoryView View;
		TestFalse(TEXT("FCbFieldView(None|Name)::TryGetView()"), NoneField.TryGetView(View));

		uint8 CopyBytes[sizeof(NoneBytes)];
		NoneField.CopyTo(MakeMemoryView(CopyBytes));
		TestTrue(TEXT("FCbFieldView(None|Name)::CopyTo()"), MakeMemoryView(NoneBytes).EqualBytes(MakeMemoryView(CopyBytes)));
	}

	// Test FCbFieldView(None|EmptyName)
	{
		constexpr ECbFieldType FieldType = ECbFieldType::None | ECbFieldType::HasFieldName;
		constexpr const uint8 NoneBytes[] = { uint8(FieldType), 0 };
		FCbFieldView NoneField(NoneBytes + 1, FieldType);
		TestEqual(TEXT("FCbFieldView(None|EmptyName)::GetSize()"), NoneField.GetSize(), uint64(sizeof(NoneBytes)));
		TestEqual(TEXT("FCbFieldView(None|EmptyName)::GetName()"), NoneField.GetName(), UTF8TEXTVIEW(""));
		TestTrue(TEXT("FCbFieldView(None|EmptyName)::HasName()"), NoneField.HasName());
		TestFalse(TEXT("!FCbFieldView(None|EmptyName)::HasValue()"), NoneField.HasValue());
		TestEqual(TEXT("FCbFieldView(None|EmptyName)::GetHash()"), NoneField.GetHash(), FIoHash::HashBuffer(MakeMemoryView(NoneBytes)));
		FMemoryView View;
		TestFalse(TEXT("FCbFieldView(None|EmptyName)::TryGetView()"), NoneField.TryGetView(View));
	}

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FCbFieldNullTest, FCbFieldTestBase, "System.Core.Serialization.CbField.Null", CompactBinaryTestFlags)
bool FCbFieldNullTest::RunTest(const FString& Parameters)
{
	// Test FCbFieldView(Null)
	{
		FCbFieldView NullField(nullptr, ECbFieldType::Null);
		TestEqual(TEXT("FCbFieldView(Null)::GetSize() == 1"), NullField.GetSize(), uint64(1));
		TestTrue(TEXT("FCbFieldView(Null)::IsNull()"), NullField.IsNull());
		TestTrue(TEXT("FCbFieldView(Null)::HasValue()"), NullField.HasValue());
		TestFalse(TEXT("!FCbFieldView(Null)::HasError()"), NullField.HasError());
		TestEqual(TEXT("FCbFieldView(Null)::GetError() == None"), NullField.GetError(), ECbFieldError::None);
		TestEqual(TEXT("FCbFieldView(Null)::GetHash()"), NullField.GetHash(), FIoHash::HashBuffer(MakeMemoryView<uint8>({uint8(ECbFieldType::Null)})));
	}

	// Test FCbFieldView(None) as Null
	{
		FCbFieldView Field;
		TestFalse(TEXT("FCbFieldView(None)::IsNull()"), Field.IsNull());
	}

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FCbFieldObjectTest, FCbFieldTestBase, "System.Core.Serialization.CbField.Object", CompactBinaryTestFlags)
bool FCbFieldObjectTest::RunTest(const FString& Parameters)
{
	static_assert(!std::is_constructible<FCbFieldView, const FCbObjectView&>::value, "Invalid constructor for FCbFieldView");
	static_assert(!std::is_assignable<FCbFieldView, const FCbObjectView&>::value, "Invalid assignment for FCbFieldView");
	static_assert(!std::is_convertible<FCbFieldView, FCbObjectView>::value, "Invalid conversion to FCbObjectView");
	static_assert(!std::is_assignable<FCbObjectView, const FCbFieldView&>::value, "Invalid assignment for FCbObjectView");

	auto TestIntObject = [this](const FCbObjectView& Object, int32 ExpectedNum, uint64 ExpectedValueSize)
	{
		TestEqual(TEXT("FCbFieldView(Object)::AsObjectView().GetSize()"), Object.GetSize(), ExpectedValueSize + sizeof(ECbFieldType));

		int32 ActualNum = 0;
		for (FCbFieldViewIterator It = Object.CreateViewIterator(); It; ++It)
		{
			++ActualNum;
			TestNotEqual(TEXT("FCbFieldView(Object) Iterator Name"), It->GetName().Len(), 0);
			TestEqual(TEXT("FCbFieldView(Object) Iterator"), It->AsInt32(), ActualNum);
		}
		TestEqual(TEXT("FCbFieldView(Object)::AsObjectView().CreateViewIterator() -> Count"), ActualNum, ExpectedNum);

		ActualNum = 0;
		for (FCbFieldView Field : Object)
		{
			++ActualNum;
			TestNotEqual(TEXT("FCbFieldView(Object) Iterator Name"), Field.GetName().Len(), 0);
			TestEqual(TEXT("FCbFieldView(Object) Range"), Field.AsInt32(), ActualNum);
		}
		TestEqual(TEXT("FCbFieldView(Object)::AsObjectView() Range -> Count"), ActualNum, ExpectedNum);

		ActualNum = 0;
		for (FCbFieldView Field : Object.AsFieldView())
		{
			++ActualNum;
			TestNotEqual(TEXT("FCbFieldView(ObjectField) Iterator Name"), Field.GetName().Len(), 0);
			TestEqual(TEXT("FCbFieldView(ObjectField) Range"), Field.AsInt32(), ActualNum);
		}
		TestEqual(TEXT("FCbFieldView(ObjectField)::AsObjectView() Range -> Count"), ActualNum, ExpectedNum);
	};

	// Test FCbFieldView(Object, Empty)
	TestFieldBytes<ECbFieldType::Object>(TEXT("Object, Empty"), {0});

	// Test FCbFieldView(Object, Empty)
	{
		FCbObjectView Object;
		TestIntObject(Object, 0, 1);

		// Find fields that do not exist.
		TestFalse(TEXT("FCbObjectView()::Find(Missing)"), Object.FindView(UTF8TEXTVIEW("Field")).HasValue());
		TestFalse(TEXT("FCbObjectView()::FindViewIgnoreCase(Missing)"), Object.FindViewIgnoreCase(UTF8TEXTVIEW("Field")).HasValue());
		TestFalse(TEXT("FCbObjectView()::operator[](Missing)"), Object[UTF8TEXTVIEW("Field")].HasValue());

		// Advance an iterator past the last field.
		FCbFieldViewIterator It = Object.CreateViewIterator();
		TestFalse(TEXT("FCbObjectView()::CreateViewIterator() At End"), bool(It));
		TestTrue(TEXT("FCbObjectView()::CreateViewIterator() At End"), !It);
		for (int Count = 16; Count > 0; --Count)
		{
			++It;
			(void)It->AsInt32();
		}
		TestFalse(TEXT("FCbObjectView()::CreateViewIterator() At End"), bool(It));
		TestTrue(TEXT("FCbObjectView()::CreateViewIterator() At End"), !It);
	}

	// Test FCbFieldView(Object, NotEmpty)
	{
		constexpr uint8 IntType = uint8(ECbFieldType::HasFieldName | ECbFieldType::IntegerPositive);
		const uint8 Value[] = { 12, IntType, 1, 'A', 1, IntType, 1, 'B', 2, IntType, 1, 'C', 3 };
		FCbFieldView Field(Value, ECbFieldType::Object);
		TestField<ECbFieldType::Object>(TEXT("Object, NotEmpty"), Field, FCbObjectView(Value, ECbFieldType::Object));
		FCbObject Object = FCbObject::Clone(Field.AsObjectView());
		TestIntObject(Object, 3, sizeof(Value));
		TestIntObject(Field.AsObjectView(), 3, sizeof(Value));
		TestTrue(TEXT("FCbObjectView::Equals()"), Object.Equals(Field.AsObjectView()));
		TestEqual(TEXT("FCbObjectView::Find()"), Object.FindView(ANSITEXTVIEW("B")).AsInt32(), 2);
		TestEqual(TEXT("FCbObjectView::Find()"), Object.FindView(ANSITEXTVIEW("b")).AsInt32(4), 4);
		TestEqual(TEXT("FCbObjectView::FindViewIgnoreCase()"), Object.FindViewIgnoreCase(ANSITEXTVIEW("B")).AsInt32(), 2);
		TestEqual(TEXT("FCbObjectView::FindViewIgnoreCase()"), Object.FindViewIgnoreCase(ANSITEXTVIEW("b")).AsInt32(), 2);
		TestEqual(TEXT("FCbObjectView::operator[]"), Object[ANSITEXTVIEW("B")].AsInt32(), 2);
		TestEqual(TEXT("FCbObjectView::operator[]"), Object[ANSITEXTVIEW("b")].AsInt32(4), 4);
	}

	// Test FCbFieldView(UniformObject, NotEmpty)
	{
		constexpr uint8 IntType = uint8(ECbFieldType::HasFieldName | ECbFieldType::IntegerPositive);
		const uint8 Value[] = { 10, IntType, 1, 'A', 1, 1, 'B', 2, 1, 'C', 3 };
		FCbFieldView Field(Value, ECbFieldType::UniformObject);
		TestField<ECbFieldType::UniformObject>(TEXT("UniformObject, NotEmpty"), Field, FCbObjectView(Value, ECbFieldType::UniformObject));
		FCbObject Object = FCbObject::Clone(Field.AsObjectView());
		TestIntObject(Object, 3, sizeof(Value));
		TestIntObject(Field.AsObjectView(), 3, sizeof(Value));
		TestTrue(TEXT("FCbObjectView(Uniform)::Equals()"), Object.Equals(Field.AsObjectView()));
		TestEqual(TEXT("FCbObjectView(Uniform)::Find()"), Object.FindView(ANSITEXTVIEW("B")).AsInt32(), 2);
		TestEqual(TEXT("FCbObjectView(Uniform)::Find()"), Object.Find(ANSITEXTVIEW("B")).AsInt32(), 2);
		TestEqual(TEXT("FCbObjectView(Uniform)::Find()"), Object.FindView(ANSITEXTVIEW("b")).AsInt32(4), 4);
		TestEqual(TEXT("FCbObjectView(Uniform)::Find()"), Object.Find(ANSITEXTVIEW("b")).AsInt32(4), 4);
		TestEqual(TEXT("FCbObjectView(Uniform)::FindViewIgnoreCase()"), Object.FindViewIgnoreCase(ANSITEXTVIEW("B")).AsInt32(), 2);
		TestEqual(TEXT("FCbObjectView(Uniform)::FindViewIgnoreCase()"), Object.FindIgnoreCase(ANSITEXTVIEW("B")).AsInt32(), 2);
		TestEqual(TEXT("FCbObjectView(Uniform)::FindViewIgnoreCase()"), Object.FindViewIgnoreCase(ANSITEXTVIEW("b")).AsInt32(), 2);
		TestEqual(TEXT("FCbObjectView(Uniform)::FindViewIgnoreCase()"), Object.FindIgnoreCase(ANSITEXTVIEW("b")).AsInt32(), 2);
		TestEqual(TEXT("FCbObjectView(Uniform)::operator[]"), Object[ANSITEXTVIEW("B")].AsInt32(), 2);
		TestEqual(TEXT("FCbObjectView(Uniform)::operator[]"), Object[ANSITEXTVIEW("b")].AsInt32(4), 4);

		TestTrue(TEXT("FCbObject::AsField()"), Object.GetOuterBuffer() == Object.AsField().AsObject().GetOuterBuffer());

		// Equals
		const uint8 NamedValue[] = { 1, 'O', 10, IntType, 1, 'A', 1, 1, 'B', 2, 1, 'C', 3 };
		FCbFieldView NamedField(NamedValue, ECbFieldType::UniformObject | ECbFieldType::HasFieldName);
		TestTrue(TEXT("FCbObjectView::Equals()"), Field.AsObjectView().Equals(NamedField.AsObjectView()));
		TestTrue(TEXT("FCbObjectView::AsFieldView().Equals()"), Field.Equals(Field.AsObjectView().AsFieldView()));
		TestTrue(TEXT("FCbObjectView::AsFieldView().Equals()"), NamedField.RemoveName().Equals(NamedField.AsObjectView().AsFieldView()));

		// CopyTo
		uint8 CopyBytes[sizeof(Value) + 1];
		Field.AsObjectView().CopyTo(MakeMemoryView(CopyBytes));
		TestTrue(TEXT("FCbObjectView(NoType)::CopyTo()"), MakeMemoryView(Value).EqualBytes(MakeMemoryView(CopyBytes) + 1));
		NamedField.AsObjectView().CopyTo(MakeMemoryView(CopyBytes));
		TestTrue(TEXT("FCbObjectView(NoType, Name)::CopyTo()"), MakeMemoryView(Value).EqualBytes(MakeMemoryView(CopyBytes) + 1));

		// TryGetView
		FMemoryView View;
		TestTrue(TEXT("FCbObjectView(Clone)::TryGetView()"), Object.TryGetView(View) && View == Object.GetOuterBuffer().GetView());
		TestFalse(TEXT("FCbObjectView(NoType)::TryGetView()"), Field.AsObjectView().TryGetView(View));
		TestFalse(TEXT("FCbObjectView(Name)::TryGetView()"), NamedField.AsObjectView().TryGetView(View));

		// GetBuffer
		TestTrue(TEXT("FCbObjectView(Clone)::GetBuffer()"), Object.GetBuffer().ToShared().GetView() == Object.GetOuterBuffer().GetView());
		TestTrue(TEXT("FCbObjectView(NoType)::GetBuffer()"), FCbField::MakeView(Field).AsObject().GetBuffer().ToShared().GetView().EqualBytes(Object.GetOuterBuffer().GetView()));
		TestTrue(TEXT("FCbObjectView(Name)::GetBuffer()"), FCbField::MakeView(NamedField).AsObject().GetBuffer().ToShared().GetView().EqualBytes(Object.GetOuterBuffer().GetView()));
	}

	// Test FCbFieldView(None) as Object
	{
		FCbFieldView FieldView;
		TestFieldError<ECbFieldType::Object>(TEXT("Object, None, View"), FieldView, ECbFieldError::TypeError);
		FCbField Field;
		TestFieldError<ECbFieldType::Object, FCbObject, FCbFieldObjectAccessors>(TEXT("Object, None"), Field, ECbFieldError::TypeError);
	}

	// Test FCbObjectView(ObjectWithName) and CreateIterator
	{
		const uint8 ObjectType = uint8(ECbFieldType::Object | ECbFieldType::HasFieldName);
		const uint8 Buffer[] = { ObjectType, 3, 'K', 'e', 'y', 4, uint8(ECbFieldType::HasFieldName | ECbFieldType::IntegerPositive), 1, 'F', 8 };
		const FCbObjectView Object(Buffer);
		TestEqual(TEXT("FCbObjectView(ObjectWithName)::GetSize()"), Object.GetSize(), uint64(6));
		const FCbObject ObjectClone = FCbObject::Clone(Object);
		TestEqual(TEXT("FCbObject(ObjectWithName)::GetSize()"), ObjectClone.GetSize(), uint64(6));
		TestTrue(TEXT("FCbObjectView::Equals()"), Object.Equals(ObjectClone));
		TestEqual(TEXT("FCbObjectView::GetHash()"), ObjectClone.GetHash(), Object.GetHash());
		for (FCbFieldIterator It = ObjectClone.CreateIterator(); It; ++It)
		{
			FCbField Field = *It;
			TestEqual(TEXT("FCbObject::CreateIterator().GetName()"), Field.GetName(), UTF8TEXTVIEW("F"));
			TestEqual(TEXT("FCbObject::CreateIterator().AsInt32()"), Field.AsInt32(), 8);
			TestTrue(TEXT("FCbObject::CreateIterator().IsOwned()"), Field.IsOwned());
		}
		for (FCbFieldIterator It = ObjectClone.CreateIterator(), End; It != End; ++It)
		{
		}
		for (FCbField Field : ObjectClone)
		{
		}

		// CopyTo
		uint8 CopyBytes[6];
		Object.CopyTo(MakeMemoryView(CopyBytes));
		TestTrue(TEXT("FCbObjectView(Name)::CopyTo()"), ObjectClone.GetOuterBuffer().GetView().EqualBytes(MakeMemoryView(CopyBytes)));
		ObjectClone.CopyTo(MakeMemoryView(CopyBytes));
		TestTrue(TEXT("FCbObjectView()::CopyTo()"), ObjectClone.GetOuterBuffer().GetView().EqualBytes(MakeMemoryView(CopyBytes)));

		// GetBuffer
		TestTrue(TEXT("FCbField(ObjectWithName)::GetBuffer()"), FCbField(FSharedBuffer::MakeView(MakeMemoryView(Buffer))).GetBuffer().ToShared().GetView().EqualBytes(MakeMemoryView(Buffer)));
		TestTrue(TEXT("FCbField(ObjectWithNameNoType)::GetBuffer()"),
			FCbField(FCbFieldView(Buffer + 1, ECbFieldType(ObjectType)), FSharedBuffer::MakeView(MakeMemoryView(Buffer))).GetBuffer().ToShared().GetView().EqualBytes(MakeMemoryView(Buffer)));

		// Access Missing Field
		TestFalse(TEXT("FCbObject()[Missing]"), ObjectClone[ANSITEXTVIEW("M")].HasValue());
		TestFalse(TEXT("FCbField(Object)[Missing]"), ObjectClone.AsField()[ANSITEXTVIEW("M")].HasValue());
	}

	// Test FCbObjectView as FCbFieldViewIterator
	{
		uint32 Count = 0;
		FCbObjectView Object;
		for (FCbFieldView Field : FCbFieldViewIterator::MakeSingle(Object.AsFieldView()))
		{
			TestTrue(TEXT("FCbObjectView::AsFieldView() as Iterator"), Field.IsObject());
			++Count;
		}
		TestEqual(TEXT("FCbObjectView::AsFieldView() as Iterator Count"), Count, 1u);
	}

	// Test FCbObject as FCbFieldIterator
	{
		uint32 Count = 0;
		FCbObject Object;
		Object.MakeOwned();
		for (FCbField Field : FCbFieldIterator::MakeSingle(Object.AsField()))
		{
			TestTrue(TEXT("FCbObject::AsFieldView() as Iterator"), Field.IsObject());
			++Count;
		}
		TestEqual(TEXT("FCbObject::AsFieldView() as Iterator Count"), Count, 1u);
	}

	// Test FCbObject(Empty) as FCbFieldIterator
	{
		const uint8 Buffer[] = { uint8(ECbFieldType::Object), 0 };
		const FCbObject Object = FCbObject::Clone(Buffer);
		for (FCbField& Field : Object)
		{
		}
	}

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FCbFieldArrayTest, FCbFieldTestBase, "System.Core.Serialization.CbField.Array", CompactBinaryTestFlags)
bool FCbFieldArrayTest::RunTest(const FString& Parameters)
{
	static_assert(!std::is_constructible<FCbFieldView, const FCbArrayView&>::value, "Invalid constructor for FCbFieldView");
	static_assert(!std::is_assignable<FCbFieldView, const FCbArrayView&>::value, "Invalid assignment for FCbFieldView");
	static_assert(!std::is_convertible<FCbFieldView, FCbArrayView>::value, "Invalid conversion to FCbArrayView");
	static_assert(!std::is_assignable<FCbArrayView, const FCbFieldView&>::value, "Invalid assignment for FCbArrayView");

	auto TestIntArray = [this](FCbArrayView Array, int32 ExpectedNum, uint64 ExpectedValueSize)
	{
		TestEqual(TEXT("FCbFieldView(Array)::AsArrayView().GetSize()"), Array.GetSize(), ExpectedValueSize + sizeof(ECbFieldType));
		TestEqual(TEXT("FCbFieldView(Array)::AsArrayView().Num()"), Array.Num(), uint64(ExpectedNum));

		int32 ActualNum = 0;
		for (FCbFieldViewIterator It = Array.CreateViewIterator(); It; ++It)
		{
			++ActualNum;
			TestEqual(TEXT("FCbFieldView(Array) Iterator"), It->AsInt32(), ActualNum);
		}
		TestEqual(TEXT("FCbFieldView(Array)::AsArrayView().CreateViewIterator() -> Count"), ActualNum, ExpectedNum);

		ActualNum = 0;
		for (FCbFieldView Field : Array)
		{
			++ActualNum;
			TestEqual(TEXT("FCbFieldView(Array) Range"), Field.AsInt32(), ActualNum);
		}
		TestEqual(TEXT("FCbFieldView(Array)::AsArrayView() Range -> Count"), ActualNum, ExpectedNum);

		ActualNum = 0;
		for (FCbFieldView Field : Array.AsFieldView())
		{
			++ActualNum;
			TestEqual(TEXT("FCbFieldView(ArrayField) Range"), Field.AsInt32(), ActualNum);
		}
		TestEqual(TEXT("FCbFieldView(ArrayField)::AsArrayView() Range -> Count"), ActualNum, ExpectedNum);
	};

	// Test FCbFieldView(Array, Empty)
	TestFieldBytes<ECbFieldType::Array>(TEXT("Array, Empty"), {1, 0});

	// Test FCbFieldView(Array, Empty)
	{
		FCbArrayView Array;
		TestIntArray(Array, 0, 2);

		// Advance an iterator past the last field.
		FCbFieldViewIterator It = Array.CreateViewIterator();
		TestFalse(TEXT("FCbArrayView()::CreateViewIterator() At End"), bool(It));
		TestTrue(TEXT("FCbArrayView()::CreateViewIterator() At End"), !It);
		for (int Count = 16; Count > 0; --Count)
		{
			++It;
			(void)It->AsInt32();
		}
		TestFalse(TEXT("FCbArrayView()::CreateViewIterator() At End"), bool(It));
		TestTrue(TEXT("FCbArrayView()::CreateViewIterator() At End"), !It);
	}

	// Test FCbFieldView(Array, NotEmpty)
	{
		constexpr uint8 IntType = uint8(ECbFieldType::IntegerPositive);
		const uint8 Value[] = { 7, 3, IntType, 1, IntType, 2, IntType, 3 };
		FCbFieldView Field(Value, ECbFieldType::Array);
		TestField<ECbFieldType::Array>(TEXT("Array, NotEmpty"), Field, FCbArrayView(Value, ECbFieldType::Array));
		FCbArray Array = FCbArray::Clone(Field.AsArrayView());
		TestIntArray(Array, 3, sizeof(Value));
		TestIntArray(Field.AsArrayView(), 3, sizeof(Value));
		TestTrue(TEXT("FCbArrayView::Equals()"), Array.Equals(Field.AsArrayView()));
	}

	// Test FCbFieldView(UniformArray)
	{
		constexpr uint8 IntType = uint8(ECbFieldType::IntegerPositive);
		const uint8 Value[] = { 5, 3, IntType, 1, 2, 3 };
		FCbFieldView Field(Value, ECbFieldType::UniformArray);
		TestField<ECbFieldType::UniformArray>(TEXT("UniformArray"), Field, FCbArrayView(Value, ECbFieldType::UniformArray));
		FCbArray Array = FCbArray::Clone(Field.AsArrayView());
		TestIntArray(Array, 3, sizeof(Value));
		TestIntArray(Field.AsArrayView(), 3, sizeof(Value));
		TestTrue(TEXT("FCbArrayView(Uniform)::Equals()"), Array.Equals(Field.AsArrayView()));

		TestTrue(TEXT("FCbArray::AsField()"), Array.GetOuterBuffer() == Array.AsField().AsArray().GetOuterBuffer());

		// Equals
		const uint8 NamedValue[] = { 1, 'A', 5, 3, IntType, 1, 2, 3 };
		FCbFieldView NamedField(NamedValue, ECbFieldType::UniformArray | ECbFieldType::HasFieldName);
		TestTrue(TEXT("FCbArrayView::Equals()"), Field.AsArrayView().Equals(NamedField.AsArrayView()));
		TestTrue(TEXT("FCbArrayView::AsFieldView().Equals()"), Field.Equals(Field.AsArrayView().AsFieldView()));
		TestTrue(TEXT("FCbArrayView::AsFieldView().Equals()"), NamedField.RemoveName().Equals(NamedField.AsArrayView().AsFieldView()));

		// CopyTo
		uint8 CopyBytes[sizeof(Value) + 1];
		Field.AsArrayView().CopyTo(MakeMemoryView(CopyBytes));
		TestTrue(TEXT("FCbArrayView(NoType)::CopyTo()"), MakeMemoryView(Value).EqualBytes(MakeMemoryView(CopyBytes) + 1));
		NamedField.AsArrayView().CopyTo(MakeMemoryView(CopyBytes));
		TestTrue(TEXT("FCbArrayView(NoType, Name)::CopyTo()"), MakeMemoryView(Value).EqualBytes(MakeMemoryView(CopyBytes) + 1));

		// TryGetView
		FMemoryView View;
		TestTrue(TEXT("FCbArrayView(Clone)::TryGetView()"), Array.TryGetView(View) && View == Array.GetOuterBuffer().GetView());
		TestFalse(TEXT("FCbArrayView(NoType)::TryGetView()"), Field.AsArrayView().TryGetView(View));
		TestFalse(TEXT("FCbArrayView(Name)::TryGetView()"), NamedField.AsArrayView().TryGetView(View));

		// GetBuffer
		TestTrue(TEXT("FCbArrayView(Clone)::GetBuffer()"), Array.GetBuffer().ToShared().GetView() == Array.GetOuterBuffer().GetView());
		TestTrue(TEXT("FCbArrayView(NoType)::GetBuffer()"), FCbField::MakeView(Field).AsArray().GetBuffer().ToShared().GetView().EqualBytes(Array.GetOuterBuffer().GetView()));
		TestTrue(TEXT("FCbArrayView(Name)::GetBuffer()"), FCbField::MakeView(NamedField).AsArray().GetBuffer().ToShared().GetView().EqualBytes(Array.GetOuterBuffer().GetView()));
	}

	// Test FCbFieldView(None) as Array
	{
		FCbFieldView FieldView;
		TestFieldError<ECbFieldType::Array>(TEXT("Array, None, View"), FieldView, ECbFieldError::TypeError);
		FCbField Field;
		TestFieldError<ECbFieldType::Array, FCbArray, FCbFieldArrayAccessors>(TEXT("Array, None"), Field, ECbFieldError::TypeError);
	}

	// Test FCbArrayView(ArrayWithName) and CreateIterator
	{
		const uint8 ArrayType = uint8(ECbFieldType::Array | ECbFieldType::HasFieldName);
		const uint8 Buffer[] = { ArrayType, 3, 'K', 'e', 'y', 3, 1, uint8(ECbFieldType::IntegerPositive), 8 };
		const FCbArrayView Array(Buffer);
		TestEqual(TEXT("FCbArrayView(ArrayWithName)::GetSize()"), Array.GetSize(), uint64(5));
		const FCbArray ArrayClone = FCbArray::Clone(Array);
		TestEqual(TEXT("FCbArray(ArrayWithName)::GetSize()"), ArrayClone.GetSize(), uint64(5));
		TestTrue(TEXT("FCbArrayView::Equals()"), Array.Equals(ArrayClone));
		TestEqual(TEXT("FCbArrayView::GetHash()"), ArrayClone.GetHash(), Array.GetHash());
		for (FCbFieldIterator It = ArrayClone.CreateIterator(); It; ++It)
		{
			FCbField Field = *It;
			TestEqual(TEXT("FCbArray::CreateIterator().AsInt32()"), Field.AsInt32(), 8);
			TestTrue(TEXT("FCbArray::CreateIterator().IsOwned()"), Field.IsOwned());
		}
		for (FCbFieldIterator It = ArrayClone.CreateIterator(), End; It != End; ++It)
		{
		}
		for (FCbField Field : ArrayClone)
		{
		}

		// CopyTo
		uint8 CopyBytes[5];
		Array.CopyTo(MakeMemoryView(CopyBytes));
		TestTrue(TEXT("FCbArrayView(Name)::CopyTo()"), ArrayClone.GetOuterBuffer().GetView().EqualBytes(MakeMemoryView(CopyBytes)));
		ArrayClone.CopyTo(MakeMemoryView(CopyBytes));
		TestTrue(TEXT("FCbArrayView()::CopyTo()"), ArrayClone.GetOuterBuffer().GetView().EqualBytes(MakeMemoryView(CopyBytes)));

		// GetBuffer
		TestTrue(TEXT("FCbField(ArrayWithName)::GetBuffer()"), FCbField(FSharedBuffer::MakeView(MakeMemoryView(Buffer))).GetBuffer().ToShared().GetView().EqualBytes(MakeMemoryView(Buffer)));
		TestTrue(TEXT("FCbField(ArrayWithNameNoType)::GetBuffer()"),
			FCbField(FCbFieldView(Buffer + 1, ECbFieldType(ArrayType)), FSharedBuffer::MakeView(MakeMemoryView(Buffer))).GetBuffer().ToShared().GetView().EqualBytes(MakeMemoryView(Buffer)));
	}

	// Test FCbArrayView as FCbFieldViewIterator
	{
		uint32 Count = 0;
		FCbArrayView Array;
		for (FCbFieldView Field : FCbFieldViewIterator::MakeSingle(Array.AsFieldView()))
		{
			TestTrue(TEXT("FCbArrayView::AsFieldView() as Iterator"), Field.IsArray());
			++Count;
		}
		TestEqual(TEXT("FCbArrayView::AsFieldView() as Iterator Count"), Count, 1u);
	}

	// Test FCbArray as FCbFieldIterator
	{
		uint32 Count = 0;
		FCbArray Array;
		Array.MakeOwned();
		for (FCbField Field : FCbFieldIterator::MakeSingle(Array.AsField()))
		{
			TestTrue(TEXT("FCbArray::AsFieldView() as Iterator"), Field.IsArray());
			++Count;
		}
		TestEqual(TEXT("FCbArray::AsFieldView() as Iterator Count"), Count, 1u);
	}

	// Test FCbArray(Empty) as FCbFieldIterator
	{
		const uint8 Buffer[] = { uint8(ECbFieldType::Array), 1, 0 };
		const FCbArray Array = FCbArray::Clone(Buffer);
		for (FCbField& Field : Array)
		{
		}
	}

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FCbFieldBinaryTest, FCbFieldTestBase, "System.Core.Serialization.CbField.Binary", CompactBinaryTestFlags)
bool FCbFieldBinaryTest::RunTest(const FString& Parameters)
{
	struct FCbBinaryAccessors
	{
		TUniqueFunction<bool (const FCbFieldView&)> IsType = [](const FCbFieldView& Field) { return Field.IsInteger(); };
		TUniqueFunction<FSharedBuffer (FCbFieldView& Field, const FSharedBuffer& Default)> AsType =
			[](FCbFieldView& Field, const FSharedBuffer& Default) -> FSharedBuffer { return static_cast<FCbField&>(Field).AsBinary(Default); };
	};

	// Test FCbFieldView(Binary, Empty)
	TestFieldBytes<ECbFieldType::Binary>(TEXT("Binary, Empty"), {0});

	// Test FCbFieldView(Binary, Value)
	{
		const uint8 Value[] = { 3, 4, 5, 6 }; // Size: 3, Data: 4/5/6
		FCbFieldView FieldView(Value, ECbFieldType::Binary);
		TestFieldNoClone<ECbFieldType::Binary>(TEXT("Binary, Value, View"), FieldView, MakeMemoryView(Value + 1, 3));

		FCbField Field = FCbField::Clone(FieldView);
		(void)Field.AsBinary();
		TestFalse(TEXT("Binary, Value, AsBinary -> GetOuterBuffer()"), Field.GetOuterBuffer().IsNull());
		(void)MoveTemp(Field).AsBinary();
		TestTrue(TEXT("Binary, Value, AsBinary -> GetOuterBuffer()"), Field.GetOuterBuffer().IsNull());
	}

	// Test FCbFieldView(None) as Binary
	{
		FCbFieldView FieldView;
		const uint8 Default[] = { 1, 2, 3 };
		TestFieldError<ECbFieldType::Binary>(TEXT("Binary, None, View"), FieldView, ECbFieldError::TypeError, MakeMemoryView(Default));

		FCbField Field = FCbField::Clone(FieldView);
		TestFieldError<ECbFieldType::Binary, FSharedBuffer>(TEXT("Binary, None"), Field, ECbFieldError::TypeError, FSharedBuffer::MakeView(MakeMemoryView(Default)), FCbBinaryAccessors());
	}

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FCbFieldStringTest, FCbFieldTestBase, "System.Core.Serialization.CbField.String", CompactBinaryTestFlags)
bool FCbFieldStringTest::RunTest(const FString& Parameters)
{
	// Test FCbFieldView(String, Empty)
	TestFieldBytes<ECbFieldType::String>(TEXT("String, Empty"), {0});

	// Test FCbFieldView(String, Value)
	{
		const uint8 Value[] = { 3, 'A', 'B', 'C' }; // Size: 3, Data: ABC
		TestFieldBytes<ECbFieldType::String>(TEXT("String, Value"), Value, FUtf8StringView(reinterpret_cast<const UTF8CHAR*>(Value) + 1, 3));
	}

	// Test FCbFieldView(String, OutOfRangeSize)
	{
		uint8 Value[9];
		WriteVarUInt(uint64(1) << 31, Value);
		TestFieldBytesError<ECbFieldType::String>(TEXT("String, OutOfRangeSize"), Value, ECbFieldError::RangeError, UTF8TEXTVIEW("ABC"));
	}

	// Test FCbFieldView(None) as String
	{
		FCbFieldView Field;
		TestFieldError<ECbFieldType::String>(TEXT("String, None"), Field, ECbFieldError::TypeError, UTF8TEXTVIEW("ABC"));
	}

	return true;
}

class FCbFieldIntegerTestBase : public FCbFieldTestBase
{
protected:
	using FCbFieldTestBase::FCbFieldTestBase;

	enum class EIntType : uint8
	{
		None   = 0x00,
		Int8   = 0x01,
		Int16  = 0x02,
		Int32  = 0x04,
		Int64  = 0x08,
		UInt8  = 0x10,
		UInt16 = 0x20,
		UInt32 = 0x40,
		UInt64 = 0x80,
		// Masks for positive values requiring the specified number of bits.
		Pos64 = UInt64,
		Pos63 = Pos64 |  Int64,
		Pos32 = Pos63 | UInt32,
		Pos31 = Pos32 |  Int32,
		Pos16 = Pos31 | UInt16,
		Pos15 = Pos16 |  Int16,
		Pos8  = Pos15 | UInt8,
		Pos7  = Pos8  |  Int8,
		// Masks for negative values requiring the specified number of bits.
		Neg63 = Int64,
		Neg31 = Neg63 | Int32,
		Neg15 = Neg31 | Int16,
		Neg7  = Neg15 | Int8,
	};

	template <typename T, T (FCbFieldView::*InAsTypeFn)(T)>
	struct TCbIntegerAccessors
	{
		static constexpr bool (FCbFieldView::*IsType)() const = &FCbFieldView::IsInteger;
		static constexpr T (FCbFieldView::*AsType)(T) = InAsTypeFn;
	};

	void TestIntegerField(ECbFieldType FieldType, EIntType ExpectedMask, uint64 Magnitude)
	{
		uint8 Value[9];
		const bool Negative = bool(uint8(FieldType) & 1);
		WriteVarUInt(Magnitude - Negative, Value);
		constexpr uint64 DefaultValue = 8;
		const uint64 ExpectedValue = Negative ? uint64(-int64(Magnitude)) : Magnitude;
		FCbFieldView Field(Value, FieldType);
		TestField<ECbFieldType::IntegerNegative>(TEXT("Int8"), Field, int8(EnumHasAnyFlags(ExpectedMask, EIntType::Int8) ? ExpectedValue : DefaultValue),
			int8(DefaultValue), EnumHasAnyFlags(ExpectedMask, EIntType::Int8) ? ECbFieldError::None : ECbFieldError::RangeError, TCbIntegerAccessors<int8, &FCbFieldView::AsInt8>());
		TestField<ECbFieldType::IntegerNegative>(TEXT("Int16"), Field, int16(EnumHasAnyFlags(ExpectedMask, EIntType::Int16) ? ExpectedValue : DefaultValue),
			int16(DefaultValue), EnumHasAnyFlags(ExpectedMask, EIntType::Int16) ? ECbFieldError::None : ECbFieldError::RangeError, TCbIntegerAccessors<int16, &FCbFieldView::AsInt16>());
		TestField<ECbFieldType::IntegerNegative>(TEXT("Int32"), Field, int32(EnumHasAnyFlags(ExpectedMask, EIntType::Int32) ? ExpectedValue : DefaultValue),
			int32(DefaultValue), EnumHasAnyFlags(ExpectedMask, EIntType::Int32) ? ECbFieldError::None : ECbFieldError::RangeError, TCbIntegerAccessors<int32, &FCbFieldView::AsInt32>());
		TestField<ECbFieldType::IntegerNegative>(TEXT("Int64"), Field, int64(EnumHasAnyFlags(ExpectedMask, EIntType::Int64) ? ExpectedValue : DefaultValue),
			int64(DefaultValue), EnumHasAnyFlags(ExpectedMask, EIntType::Int64) ? ECbFieldError::None : ECbFieldError::RangeError, TCbIntegerAccessors<int64, &FCbFieldView::AsInt64>());
		TestField<ECbFieldType::IntegerPositive>(TEXT("UInt8"), Field, uint8(EnumHasAnyFlags(ExpectedMask, EIntType::UInt8) ? ExpectedValue : DefaultValue),
			uint8(DefaultValue), EnumHasAnyFlags(ExpectedMask, EIntType::UInt8) ? ECbFieldError::None : ECbFieldError::RangeError, TCbIntegerAccessors<uint8, &FCbFieldView::AsUInt8>());
		TestField<ECbFieldType::IntegerPositive>(TEXT("UInt16"), Field, uint16(EnumHasAnyFlags(ExpectedMask, EIntType::UInt16) ? ExpectedValue : DefaultValue),
			uint16(DefaultValue), EnumHasAnyFlags(ExpectedMask, EIntType::UInt16) ? ECbFieldError::None : ECbFieldError::RangeError, TCbIntegerAccessors<uint16, &FCbFieldView::AsUInt16>());
		TestField<ECbFieldType::IntegerPositive>(TEXT("UInt32"), Field, uint32(EnumHasAnyFlags(ExpectedMask, EIntType::UInt32) ? ExpectedValue : DefaultValue),
			uint32(DefaultValue), EnumHasAnyFlags(ExpectedMask, EIntType::UInt32) ? ECbFieldError::None : ECbFieldError::RangeError, TCbIntegerAccessors<uint32, &FCbFieldView::AsUInt32>());
		TestField<ECbFieldType::IntegerPositive>(TEXT("UInt64"), Field, uint64(EnumHasAnyFlags(ExpectedMask, EIntType::UInt64) ? ExpectedValue : DefaultValue),
			uint64(DefaultValue), EnumHasAnyFlags(ExpectedMask, EIntType::UInt64) ? ECbFieldError::None : ECbFieldError::RangeError, TCbIntegerAccessors<uint64, &FCbFieldView::AsUInt64>());
	}
};

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FCbFieldIntegerTest, FCbFieldIntegerTestBase, "System.Core.Serialization.CbField.Integer", CompactBinaryTestFlags)
bool FCbFieldIntegerTest::RunTest(const FString& Parameters)
{
	// Test FCbFieldView(IntegerPositive)
	TestIntegerField(ECbFieldType::IntegerPositive, EIntType::Pos7,  0x00);
	TestIntegerField(ECbFieldType::IntegerPositive, EIntType::Pos7,  0x7f);
	TestIntegerField(ECbFieldType::IntegerPositive, EIntType::Pos8,  0x80);
	TestIntegerField(ECbFieldType::IntegerPositive, EIntType::Pos8,  0xff);
	TestIntegerField(ECbFieldType::IntegerPositive, EIntType::Pos15, 0x0100);
	TestIntegerField(ECbFieldType::IntegerPositive, EIntType::Pos15, 0x7fff);
	TestIntegerField(ECbFieldType::IntegerPositive, EIntType::Pos16, 0x8000);
	TestIntegerField(ECbFieldType::IntegerPositive, EIntType::Pos16, 0xffff);
	TestIntegerField(ECbFieldType::IntegerPositive, EIntType::Pos31, 0x0001'0000);
	TestIntegerField(ECbFieldType::IntegerPositive, EIntType::Pos31, 0x7fff'ffff);
	TestIntegerField(ECbFieldType::IntegerPositive, EIntType::Pos32, 0x8000'0000);
	TestIntegerField(ECbFieldType::IntegerPositive, EIntType::Pos32, 0xffff'ffff);
	TestIntegerField(ECbFieldType::IntegerPositive, EIntType::Pos63, 0x0000'0001'0000'0000);
	TestIntegerField(ECbFieldType::IntegerPositive, EIntType::Pos63, 0x7fff'ffff'ffff'ffff);
	TestIntegerField(ECbFieldType::IntegerPositive, EIntType::Pos64, 0x8000'0000'0000'0000);
	TestIntegerField(ECbFieldType::IntegerPositive, EIntType::Pos64, 0xffff'ffff'ffff'ffff);

	// Test FCbFieldView(IntegerNegative)
	TestIntegerField(ECbFieldType::IntegerNegative, EIntType::Neg7,  0x01);
	TestIntegerField(ECbFieldType::IntegerNegative, EIntType::Neg7,  0x80);
	TestIntegerField(ECbFieldType::IntegerNegative, EIntType::Neg15, 0x81);
	TestIntegerField(ECbFieldType::IntegerNegative, EIntType::Neg15, 0x8000);
	TestIntegerField(ECbFieldType::IntegerNegative, EIntType::Neg31, 0x8001);
	TestIntegerField(ECbFieldType::IntegerNegative, EIntType::Neg31, 0x8000'0000);
	TestIntegerField(ECbFieldType::IntegerNegative, EIntType::Neg63, 0x8000'0001);
	TestIntegerField(ECbFieldType::IntegerNegative, EIntType::Neg63, 0x8000'0000'0000'0000);
	TestIntegerField(ECbFieldType::IntegerNegative, EIntType::None,  0x8000'0000'0000'0001);
	TestIntegerField(ECbFieldType::IntegerNegative, EIntType::None,  0xffff'ffff'ffff'ffff);

	// Test FCbFieldView(None) as Integer
	{
		FCbFieldView Field;
		TestFieldError<ECbFieldType::IntegerPositive>(TEXT("Integer+, None"), Field, ECbFieldError::TypeError, uint64(8));
		TestFieldError<ECbFieldType::IntegerNegative>(TEXT("Integer-, None"), Field, ECbFieldError::TypeError, int64(8));
	}

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FCbFieldFloatTest, FCbFieldTestBase, "System.Core.Serialization.CbField.Float", CompactBinaryTestFlags)
bool FCbFieldFloatTest::RunTest(const FString& Parameters)
{
	// Test FCbFieldView(Float, 32-bit)
	{
		const uint8 Value[] = { 0xc0, 0x12, 0x34, 0x56 }; // -2.28444433f
		TestFieldBytes<ECbFieldType::Float32>(TEXT("Float32"), Value, -2.28444433f);

		FCbFieldView Field(Value, ECbFieldType::Float32);
		TestField<ECbFieldType::Float64>(TEXT("Float32, AsDouble"), Field, -2.28444433);
	}

	// Test FCbFieldView(Float, 64-bit)
	{
		const uint8 Value[] = { 0xc1, 0x23, 0x45, 0x67, 0x89, 0xab, 0xcd, 0xef }; // -631475.76888888876
		TestFieldBytes<ECbFieldType::Float64>(TEXT("Float64"), Value, -631475.76888888876);

		FCbFieldView Field(Value, ECbFieldType::Float64);
		TestFieldError<ECbFieldType::Float32>(TEXT("Float64, AsFloat"), Field, ECbFieldError::RangeError, 8.0f);
	}

	// Test FCbFieldView(Integer+, MaxBinary32) as Float
	{
		uint8 Value[9];
		WriteVarUInt((uint64(1) << 24) - 1, Value); // 16,777,215
		FCbFieldView Field(Value, ECbFieldType::IntegerPositive);
		TestField<ECbFieldType::Float32>(TEXT("Integer+, MaxBinary32, AsFloat"), Field, 16'777'215.0f);
		TestField<ECbFieldType::Float64>(TEXT("Integer+, MaxBinary32, AsDouble"), Field, 16'777'215.0);
	}

	// Test FCbFieldView(Integer+, MaxBinary32+1) as Float
	{
		uint8 Value[9];
		WriteVarUInt(uint64(1) << 24, Value); // 16,777,216
		FCbFieldView Field(Value, ECbFieldType::IntegerPositive);
		TestFieldError<ECbFieldType::Float32>(TEXT("Integer+, MaxBinary32+1, AsFloat"), Field, ECbFieldError::RangeError, 8.0f);
		TestField<ECbFieldType::Float64>(TEXT("Integer+, MaxBinary32+1, AsDouble"), Field, 16'777'216.0);
	}

	// Test FCbFieldView(Integer+, MaxBinary64) as Float
	{
		uint8 Value[9];
		WriteVarUInt((uint64(1) << 53) - 1, Value); // 9,007,199,254,740,991
		FCbFieldView Field(Value, ECbFieldType::IntegerPositive);
		TestFieldError<ECbFieldType::Float32>(TEXT("Integer+, MaxBinary64, AsFloat"), Field, ECbFieldError::RangeError, 8.0f);
		TestField<ECbFieldType::Float64>(TEXT("Integer+, MaxBinary64, AsDouble"), Field, 9'007'199'254'740'991.0);
	}

	// Test FCbFieldView(Integer+, MaxBinary64+1) as Float
	{
		uint8 Value[9];
		WriteVarUInt(uint64(1) << 53, Value); // 9,007,199,254,740,992
		FCbFieldView Field(Value, ECbFieldType::IntegerPositive);
		TestFieldError<ECbFieldType::Float32>(TEXT("Integer+, MaxBinary64+1, AsFloat"), Field, ECbFieldError::RangeError, 8.0f);
		TestFieldError<ECbFieldType::Float64>(TEXT("Integer+, MaxBinary64+1, AsDouble"), Field, ECbFieldError::RangeError, 8.0);
	}

	// Test FCbFieldView(Integer+, MaxUInt64) as Float
	{
		uint8 Value[9];
		WriteVarUInt(uint64(-1), Value); // Max uint64
		FCbFieldView Field(Value, ECbFieldType::IntegerPositive);
		TestFieldError<ECbFieldType::Float32>(TEXT("Integer+, MaxUInt64, AsFloat"), Field, ECbFieldError::RangeError, 8.0f);
		TestFieldError<ECbFieldType::Float64>(TEXT("Integer+, MaxUInt64, AsDouble"), Field, ECbFieldError::RangeError, 8.0);
	}

	// Test FCbFieldView(Integer-, MaxBinary32) as Float
	{
		uint8 Value[9];
		WriteVarUInt((uint64(1) << 24) - 2, Value); // -16,777,215
		FCbFieldView Field(Value, ECbFieldType::IntegerNegative);
		TestField<ECbFieldType::Float32>(TEXT("Integer-, MaxBinary32, AsFloat"), Field, -16'777'215.0f);
		TestField<ECbFieldType::Float64>(TEXT("Integer-, MaxBinary32, AsDouble"), Field, -16'777'215.0);
	}

	// Test FCbFieldView(Integer-, MaxBinary32+1) as Float
	{
		uint8 Value[9];
		WriteVarUInt((uint64(1) << 24) - 1, Value); // -16,777,216
		FCbFieldView Field(Value, ECbFieldType::IntegerNegative);
		TestFieldError<ECbFieldType::Float32>(TEXT("Integer-, MaxBinary32+1, AsFloat"), Field, ECbFieldError::RangeError, 8.0f);
		TestField<ECbFieldType::Float64>(TEXT("Integer-, MaxBinary32+1, AsDouble"), Field, -16'777'216.0);
	}

	// Test FCbFieldView(Integer-, MaxBinary64) as Float
	{
		uint8 Value[9];
		WriteVarUInt((uint64(1) << 53) - 2, Value); // -9,007,199,254,740,991
		FCbFieldView Field(Value, ECbFieldType::IntegerNegative);
		TestFieldError<ECbFieldType::Float32>(TEXT("Integer-, MaxBinary64, AsFloat"), Field, ECbFieldError::RangeError, 8.0f);
		TestField<ECbFieldType::Float64>(TEXT("Integer-, MaxBinary64, AsDouble"), Field, -9'007'199'254'740'991.0);
	}

	// Test FCbFieldView(Integer-, MaxBinary64+1) as Float
	{
		uint8 Value[9];
		WriteVarUInt((uint64(1) << 53) - 1, Value); // -9,007,199,254,740,992
		FCbFieldView Field(Value, ECbFieldType::IntegerNegative);
		TestFieldError<ECbFieldType::Float32>(TEXT("Integer-, MaxBinary64+1, AsFloat"), Field, ECbFieldError::RangeError, 8.0f);
		TestFieldError<ECbFieldType::Float64>(TEXT("Integer-, MaxBinary64+1, AsDouble"), Field, ECbFieldError::RangeError, 8.0);
	}

	// Test FCbFieldView(None) as Float
	{
		FCbFieldView Field;
		TestFieldError<ECbFieldType::Float32>(TEXT("None, AsFloat"), Field, ECbFieldError::TypeError, 8.0f);
		TestFieldError<ECbFieldType::Float64>(TEXT("None, AsDouble"), Field, ECbFieldError::TypeError, 8.0);
	}

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FCbFieldBoolTest, FCbFieldTestBase, "System.Core.Serialization.CbField.Bool", CompactBinaryTestFlags)
bool FCbFieldBoolTest::RunTest(const FString& Parameters)
{
	// Test FCbFieldView(Bool, False)
	TestFieldBytes<ECbFieldType::BoolFalse>(TEXT("Bool, False"), {}, false, true);

	// Test FCbFieldView(Bool, True)
	TestFieldBytes<ECbFieldType::BoolTrue>(TEXT("Bool, True"), {}, true, false);

	// Test FCbFieldView(None) as Bool
	{
		FCbFieldView DefaultField;
		TestFieldError<ECbFieldType::BoolFalse>(TEXT("Bool, False, None"), DefaultField, ECbFieldError::TypeError, false);
		TestFieldError<ECbFieldType::BoolTrue>(TEXT("Bool, True, None"), DefaultField, ECbFieldError::TypeError, true);
	}

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FCbFieldObjectAttachmentTest, FCbFieldTestBase, "System.Core.Serialization.CbField.ObjectAttachment", CompactBinaryTestFlags)
bool FCbFieldObjectAttachmentTest::RunTest(const FString& Parameters)
{
	const FIoHash::ByteArray ZeroBytes{};
	const FIoHash::ByteArray SequentialBytes{1, 2, 3, 4, 5, 6, 7, 8, 9, 10, 11, 12, 13, 14, 15, 16, 17, 18, 19, 20};

	// Test FCbFieldView(ObjectAttachment, Zero)
	TestFieldBytes<ECbFieldType::ObjectAttachment>(TEXT("ObjectAttachment, Zero"), ZeroBytes);

	// Test FCbFieldView(ObjectAttachment, NonZero)
	TestFieldBytes<ECbFieldType::ObjectAttachment>(TEXT("ObjectAttachment, NonZero"), SequentialBytes, FIoHash(SequentialBytes));

	// Test FCbFieldView(ObjectAttachment, NonZero) AsAttachment
	{
		FCbFieldView Field(SequentialBytes, ECbFieldType::ObjectAttachment);
		TestField<ECbFieldType::ObjectAttachment>(TEXT("ObjectAttachment, NonZero, AsAttachment"), Field, FIoHash(SequentialBytes), FIoHash(), ECbFieldError::None, FCbAttachmentAccessors());
	}

	// Test FCbFieldView(None) as ObjectAttachment
	{
		FCbFieldView DefaultField;
		TestFieldError<ECbFieldType::ObjectAttachment>(TEXT("ObjectAttachment, None"), DefaultField, ECbFieldError::TypeError, FIoHash(SequentialBytes));
	}

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FCbFieldBinaryAttachmentTest, FCbFieldTestBase, "System.Core.Serialization.CbField.BinaryAttachment", CompactBinaryTestFlags)
bool FCbFieldBinaryAttachmentTest::RunTest(const FString& Parameters)
{
	const FIoHash::ByteArray ZeroBytes{};
	const FIoHash::ByteArray SequentialBytes{1, 2, 3, 4, 5, 6, 7, 8, 9, 10, 11, 12, 13, 14, 15, 16, 17, 18, 19, 20};

	// Test FCbFieldView(BinaryAttachment, Zero)
	TestFieldBytes<ECbFieldType::BinaryAttachment>(TEXT("BinaryAttachment, Zero"), ZeroBytes);

	// Test FCbFieldView(BinaryAttachment, NonZero)
	TestFieldBytes<ECbFieldType::BinaryAttachment>(TEXT("BinaryAttachment, NonZero"), SequentialBytes, FIoHash(SequentialBytes));

	// Test FCbFieldView(BinaryAttachment, NonZero) AsAttachment
	{
		FCbFieldView Field(SequentialBytes, ECbFieldType::BinaryAttachment);
		TestField<ECbFieldType::BinaryAttachment>(TEXT("BinaryAttachment, NonZero, AsAttachment"), Field, FIoHash(SequentialBytes), FIoHash(), ECbFieldError::None, FCbAttachmentAccessors());
	}

	// Test FCbFieldView(None) as BinaryAttachment
	{
		FCbFieldView DefaultField;
		TestFieldError<ECbFieldType::BinaryAttachment>(TEXT("BinaryAttachment, None"), DefaultField, ECbFieldError::TypeError, FIoHash(SequentialBytes));
	}

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FCbFieldHashTest, FCbFieldTestBase, "System.Core.Serialization.CbField.Hash", CompactBinaryTestFlags)
bool FCbFieldHashTest::RunTest(const FString& Parameters)
{
	const FIoHash::ByteArray ZeroBytes{};
	const FIoHash::ByteArray SequentialBytes{1, 2, 3, 4, 5, 6, 7, 8, 9, 10, 11, 12, 13, 14, 15, 16, 17, 18, 19, 20};

	// Test FCbFieldView(Hash, Zero)
	TestFieldBytes<ECbFieldType::Hash>(TEXT("Hash, Zero"), ZeroBytes);

	// Test FCbFieldView(Hash, NonZero)
	TestFieldBytes<ECbFieldType::Hash>(TEXT("Hash, NonZero"), SequentialBytes, FIoHash(SequentialBytes));

	// Test FCbFieldView(None) as Hash
	{
		FCbFieldView DefaultField;
		TestFieldError<ECbFieldType::Hash>(TEXT("Hash, None"), DefaultField, ECbFieldError::TypeError, FIoHash(SequentialBytes));
	}

	// Test FCbFieldView(ObjectAttachment) as Hash
	{
		FCbFieldView Field(SequentialBytes, ECbFieldType::ObjectAttachment);
		TestField<ECbFieldType::Hash>(TEXT("ObjectAttachment, NonZero, AsHash"), Field, FIoHash(SequentialBytes));
	}

	// Test FCbFieldView(BinaryAttachment) as Hash
	{
		FCbFieldView Field(SequentialBytes, ECbFieldType::BinaryAttachment);
		TestField<ECbFieldType::Hash>(TEXT("BinaryAttachment, NonZero, AsHash"), Field, FIoHash(SequentialBytes));
	}

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FCbFieldUuidTest, FCbFieldTestBase, "System.Core.Serialization.CbField.Uuid", CompactBinaryTestFlags)
bool FCbFieldUuidTest::RunTest(const FString& Parameters)
{
	const uint8 ZeroBytes[]{0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0};
	const uint8 SequentialBytes[]{0, 1, 2, 3, 4, 5, 6, 7, 8, 9, 10, 11, 12, 13, 14, 15};
	const FGuid SequentialGuid(TEXT("00010203-0405-0607-0809-0a0b0c0d0e0f"));

	// Test FCbFieldView(Uuid, Zero)
	TestFieldBytes<ECbFieldType::Uuid>(TEXT("Uuid, Zero"), ZeroBytes, FGuid(), SequentialGuid);

	// Test FCbFieldView(Uuid, NonZero)
	TestFieldBytes<ECbFieldType::Uuid>(TEXT("Uuid, NonZero"), SequentialBytes, SequentialGuid, FGuid());

	// Test FCbFieldView(None) as Uuid
	{
		FCbFieldView DefaultField;
		TestFieldError<ECbFieldType::Uuid>(TEXT("Uuid, None"), DefaultField, ECbFieldError::TypeError, FGuid::NewGuid());
	}

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FCbFieldDateTimeTest, FCbFieldTestBase, "System.Core.Serialization.CbField.DateTime", CompactBinaryTestFlags)
bool FCbFieldDateTimeTest::RunTest(const FString& Parameters)
{
	// Test FCbFieldView(DateTime, Zero)
	TestFieldBytes<ECbFieldType::DateTime>(TEXT("DateTime, Zero"), {0, 0, 0, 0, 0, 0, 0, 0});

	// Test FCbFieldView(DateTime, 0x1020'3040'5060'7080)
	TestFieldBytes<ECbFieldType::DateTime>(TEXT("DateTime, NonZero"), {0x10, 0x20, 0x30, 0x40, 0x50, 0x60, 0x70, 0x80}, int64(0x1020'3040'5060'7080));

	// Test FCbFieldView(DateTime, Zero) as FDateTime
	{
		const uint8 Value[] = {0, 0, 0, 0, 0, 0, 0, 0};
		FCbFieldView Field(Value, ECbFieldType::DateTime);
		TestEqual(TEXT("FCbFieldView()::AsDateTime()"), Field.AsDateTime(), FDateTime(0));
	}

	// Test FCbFieldView(None) as DateTime
	{
		FCbFieldView DefaultField;
		TestFieldError<ECbFieldType::DateTime>(TEXT("DateTime, None"), DefaultField, ECbFieldError::TypeError);
		const FDateTime DefaultValue(0x1020'3040'5060'7080);
		TestEqual(TEXT("FCbFieldView()::AsDateTime()"), DefaultField.AsDateTime(DefaultValue), DefaultValue);
	}

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FCbFieldTimeSpanTest, FCbFieldTestBase, "System.Core.Serialization.CbField.TimeSpan", CompactBinaryTestFlags)
bool FCbFieldTimeSpanTest::RunTest(const FString& Parameters)
{
	// Test FCbFieldView(TimeSpan, Zero)
	TestFieldBytes<ECbFieldType::TimeSpan>(TEXT("TimeSpan, Zero"), {0, 0, 0, 0, 0, 0, 0, 0});

	// Test FCbFieldView(TimeSpan, 0x1020'3040'5060'7080)
	TestFieldBytes<ECbFieldType::TimeSpan>(TEXT("TimeSpan, NonZero"), {0x10, 0x20, 0x30, 0x40, 0x50, 0x60, 0x70, 0x80}, int64(0x1020'3040'5060'7080));

	// Test FCbFieldView(TimeSpan, Zero) as FTimeSpan
	{
		const uint8 Value[] = {0, 0, 0, 0, 0, 0, 0, 0};
		FCbFieldView Field(Value, ECbFieldType::TimeSpan);
		TestEqual(TEXT("FCbFieldView()::AsTimeSpan()"), Field.AsTimeSpan(), FTimespan(0));
	}

	// Test FCbFieldView(None) as TimeSpan
	{
		FCbFieldView DefaultField;
		TestFieldError<ECbFieldType::TimeSpan>(TEXT("TimeSpan, None"), DefaultField, ECbFieldError::TypeError);
		const FTimespan DefaultValue(0x1020'3040'5060'7080);
		TestEqual(TEXT("FCbFieldView()::AsTimeSpan()"), DefaultField.AsTimeSpan(DefaultValue), DefaultValue);
	}

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FCbFieldObjectIdTest, FCbFieldTestBase, "System.Core.Serialization.CbField.ObjectId", CompactBinaryTestFlags)
bool FCbFieldObjectIdTest::RunTest(const FString& Parameters)
{
	// Test FCbFieldView(ObjectId, Zero)
	TestFieldBytes<ECbFieldType::ObjectId>(TEXT("ObjectId, Zero"), {0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0});

	// Test FCbFieldView(ObjectId, 0x102030405060708090A0B0C0)
	TestFieldBytes<ECbFieldType::ObjectId>(TEXT("ObjectId, NonZero"), {0x10, 0x20, 0x30, 0x40, 0x50, 0x60, 0x70, 0x80, 0x90, 0xA0, 0xB0, 0xC0},
		FCbObjectId(MakeMemoryView<uint8>({0x10, 0x20, 0x30, 0x40, 0x50, 0x60, 0x70, 0x80, 0x90, 0xA0, 0xB0, 0xC0})));

	// Test FCbFieldView(ObjectId, Zero) as FCbObjectId
	{
		const uint8 Value[] = {0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0};
		FCbFieldView Field(Value, ECbFieldType::ObjectId);
		TestEqual(TEXT("FCbFieldView(ObjectId, Zero)::AsObjectId()"), Field.AsObjectId(), FCbObjectId());
	}

	// Test FCbFieldView(None) as ObjectId
	{
		FCbFieldView DefaultField;
		TestFieldError<ECbFieldType::ObjectId>(TEXT("ObjectId, None"), DefaultField, ECbFieldError::TypeError);
		const FCbObjectId DefaultValue(MakeMemoryView<uint8>({0x10, 0x20, 0x30, 0x40, 0x50, 0x60, 0x70, 0x80, 0x90, 0xA0, 0xB0, 0xC0}));
		TestEqual(TEXT("FCbFieldView(None)::AsObjectId()"), DefaultField.AsObjectId(DefaultValue), DefaultValue);
	}

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FCbFieldCustomByIdTest, FCbFieldTestBase, "System.Core.Serialization.CbField.CustomById", CompactBinaryTestFlags)
bool FCbFieldCustomByIdTest::RunTest(const FString& Parameters)
{
	struct FCustomByIdAccessor
	{
		explicit FCustomByIdAccessor(uint64 Id)
			: AsType([Id](FCbFieldView& Field, FMemoryView Default) { return Field.AsCustom(Id, Default); })
		{
		}

		bool (FCbFieldView::*IsType)() const = &FCbFieldView::IsCustomById;
		TUniqueFunction<FMemoryView (FCbFieldView& Field, FMemoryView Default)> AsType;
	};

	// Test FCbFieldView(CustomById, MinId, Empty)
	{
		const uint8 Value[] = {1, 0};
		TestFieldBytes<ECbFieldType::CustomById>(TEXT("CustomById, MinId, Empty"), Value, FCbCustomById{0});
		TestFieldBytes<ECbFieldType::CustomById>(TEXT("CustomById, MinId, Empty, View"), Value, FMemoryView(), MakeMemoryView<uint8>({1, 2, 3}), ECbFieldError::None, FCustomByIdAccessor(0));
		TestFieldBytesError<ECbFieldType::CustomById>(TEXT("CustomById, MinId, Empty, InvalidId"), Value, ECbFieldError::RangeError, MakeMemoryView<uint8>({1, 2, 3}), FCustomByIdAccessor(MAX_uint64));
	}

	// Test FCbFieldView(CustomById, MinId, Value)
	{
		const uint8 Value[] = {5, 0, 1, 2, 3, 4};
		TestFieldBytesNoClone<ECbFieldType::CustomById>(TEXT("CustomById, MinId, Value"), Value, FCbCustomById{0, MakeMemoryView(Value).Right(4)});
		TestFieldBytesNoClone<ECbFieldType::CustomById>(TEXT("CustomById, MinId, Value, View"), Value, MakeMemoryView(Value).Right(4), FMemoryView(), ECbFieldError::None, FCustomByIdAccessor(0));
		TestFieldBytesError<ECbFieldType::CustomById>(TEXT("CustomById, MinId, Value, InvalidId"), Value, ECbFieldError::RangeError, MakeMemoryView<uint8>({1, 2, 3}), FCustomByIdAccessor(MAX_uint64));
	}

	// Test FCbFieldView(CustomById, MaxId, Empty)
	{
		const uint8 Value[] = {9, 0xff, 0xff, 0xff, 0xff, 0xff, 0xff, 0xff, 0xff, 0xff};
		TestFieldBytes<ECbFieldType::CustomById>(TEXT("CustomById, MaxId, Empty"), Value, FCbCustomById{MAX_uint64});
		TestFieldBytes<ECbFieldType::CustomById>(TEXT("CustomById, MaxId, Empty, View"), Value, FMemoryView(), MakeMemoryView<uint8>({1, 2, 3}), ECbFieldError::None, FCustomByIdAccessor(MAX_uint64));
		TestFieldBytesError<ECbFieldType::CustomById>(TEXT("CustomById, MaxId, Empty, InvalidId"), Value, ECbFieldError::RangeError, MakeMemoryView<uint8>({1, 2, 3}), FCustomByIdAccessor(0));
	}

	// Test FCbFieldView(CustomById, MaxId, Value)
	{
		const uint8 Value[] = {13, 0xff, 0xff, 0xff, 0xff, 0xff, 0xff, 0xff, 0xff, 0xff, 1, 2, 3, 4};
		TestFieldBytesNoClone<ECbFieldType::CustomById>(TEXT("CustomById, MaxId, Value"), Value, FCbCustomById{MAX_uint64, MakeMemoryView(Value).Right(4)});
		TestFieldBytesNoClone<ECbFieldType::CustomById>(TEXT("CustomById, MaxId, Value, View"), Value, MakeMemoryView(Value).Right(4), FMemoryView(), ECbFieldError::None, FCustomByIdAccessor(MAX_uint64));
		TestFieldBytesError<ECbFieldType::CustomById>(TEXT("CustomById, MaxId, Value, InvalidId"), Value, ECbFieldError::RangeError, MakeMemoryView<uint8>({1, 2, 3}), FCustomByIdAccessor(0));
	}

	// Test FCbFieldView(None) as CustomById
	{
		FCbFieldView DefaultField;
		TestFieldError<ECbFieldType::CustomById>(TEXT("CustomById, None"), DefaultField, ECbFieldError::TypeError, FCbCustomById{4, MakeMemoryView<uint8>({1, 2, 3})});
		TestFieldError<ECbFieldType::CustomById>(TEXT("CustomById, None, View"), DefaultField, ECbFieldError::TypeError, MakeMemoryView<uint8>({1, 2, 3}), FCustomByIdAccessor(0));
		const uint8 DefaultValue[] = {1, 2, 3};
		TestEqual(TEXT("FCbFieldView()::AsCustom(Id)"), DefaultField.AsCustom(0, MakeMemoryView(DefaultValue)), MakeMemoryView(DefaultValue));
	}

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FCbFieldCustomByNameTest, FCbFieldTestBase, "System.Core.Serialization.CbField.CustomByName", CompactBinaryTestFlags)
bool FCbFieldCustomByNameTest::RunTest(const FString& Parameters)
{
	struct FCustomByNameAccessor
	{
		explicit FCustomByNameAccessor(FUtf8StringView Name)
			: AsType([Name = FString(Name)](FCbFieldView& Field, FMemoryView Default) { return Field.AsCustom(FTCHARToUTF8(Name), Default); })
		{
		}

		bool (FCbFieldView::*IsType)() const = &FCbFieldView::IsCustomByName;
		TUniqueFunction<FMemoryView (FCbFieldView& Field, FMemoryView Default)> AsType;
	};

	// Test FCbFieldView(CustomByName, ABC, Empty)
	{
		const uint8 Value[] = {4, 3, 'A', 'B', 'C'};
		TestFieldBytes<ECbFieldType::CustomByName>(TEXT("CustomByName, MinId, Empty"), Value, FCbCustomByName{UTF8TEXTVIEW("ABC")});
		TestFieldBytes<ECbFieldType::CustomByName>(TEXT("CustomByName, MinId, Empty, View"), Value, FMemoryView(), MakeMemoryView<uint8>({1, 2, 3}), ECbFieldError::None, FCustomByNameAccessor(UTF8TEXTVIEW("ABC")));
		TestFieldBytesError<ECbFieldType::CustomByName>(TEXT("CustomByName, MinId, Empty, InvalidCase"), Value, ECbFieldError::RangeError, MakeMemoryView<uint8>({1, 2, 3}), FCustomByNameAccessor(UTF8TEXTVIEW("abc")));
	}

	// Test FCbFieldView(CustomByName, ABC, Value)
	{
		const uint8 Value[] = {8, 3, 'A', 'B', 'C', 1, 2, 3, 4};
		TestFieldBytesNoClone<ECbFieldType::CustomByName>(TEXT("CustomByName, MinId, Value"), Value, FCbCustomByName{UTF8TEXTVIEW("ABC"), MakeMemoryView(Value).Right(4)});
		TestFieldBytesNoClone<ECbFieldType::CustomByName>(TEXT("CustomByName, MinId, Value, View"), Value, MakeMemoryView(Value).Right(4), FMemoryView(), ECbFieldError::None, FCustomByNameAccessor(UTF8TEXTVIEW("ABC")));
		TestFieldBytesError<ECbFieldType::CustomByName>(TEXT("CustomByName, MinId, Value, InvalidCase"), Value, ECbFieldError::RangeError, MakeMemoryView<uint8>({1, 2, 3}), FCustomByNameAccessor(UTF8TEXTVIEW("abc")));
	}

	// Test FCbFieldView(None) as CustomByName
	{
		FCbFieldView DefaultField;
		TestFieldError<ECbFieldType::CustomByName>(TEXT("CustomByName, None"), DefaultField, ECbFieldError::TypeError, FCbCustomByName{UTF8TEXTVIEW("ABC"), MakeMemoryView<uint8>({1, 2, 3})});
		TestFieldError<ECbFieldType::CustomByName>(TEXT("CustomByName, None, View"), DefaultField, ECbFieldError::TypeError, MakeMemoryView<uint8>({1, 2, 3}), FCustomByNameAccessor(UTF8TEXTVIEW("ABC")));
		const uint8 DefaultValue[] = {1, 2, 3};
		TestEqual(TEXT("FCbFieldView()::AsCustom(Name)"), DefaultField.AsCustom(UTF8TEXTVIEW("ABC"), MakeMemoryView(DefaultValue)), MakeMemoryView(DefaultValue));
	}

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FCbFieldIterateAttachmentsTest, FCbFieldTestBase, "System.Core.Serialization.CbField.IterateAttachments", CompactBinaryTestFlags)
bool FCbFieldIterateAttachmentsTest::RunTest(const FString& Parameters)
{
	const auto MakeTestHash = [](uint32 Index) { return FIoHash::HashBuffer(&Index, sizeof(Index)); };

	FCbFieldIterator Fields;
	{
		FCbWriter Writer;

		Writer.SetName("IgnoredTypeInRoot").AddHash(MakeTestHash(100));
		Writer.AddObjectAttachment(MakeTestHash(0));
		Writer.AddBinaryAttachment(MakeTestHash(1));
		Writer.SetName("ObjAttachmentInRoot").AddObjectAttachment(MakeTestHash(2));
		Writer.SetName("BinAttachmentInRoot").AddBinaryAttachment(MakeTestHash(3));

		// Uniform array of type to ignore.
		Writer.BeginArray();
		{
			Writer << 1;
			Writer << 2;
		}
		Writer.EndArray();
		// Uniform array of binary attachments.
		Writer.BeginArray();
		{
			Writer.AddBinaryAttachment(MakeTestHash(4));
			Writer.AddBinaryAttachment(MakeTestHash(5));
		}
		Writer.EndArray();
		// Uniform array of uniform arrays.
		Writer.BeginArray();
		{
			Writer.BeginArray();
			Writer.AddBinaryAttachment(MakeTestHash(6));
			Writer.AddBinaryAttachment(MakeTestHash(7));
			Writer.EndArray();
			Writer.BeginArray();
			Writer.AddBinaryAttachment(MakeTestHash(8));
			Writer.AddBinaryAttachment(MakeTestHash(9));
			Writer.EndArray();
		}
		Writer.EndArray();
		// Uniform array of non-uniform arrays.
		Writer.BeginArray();
		{
			Writer.BeginArray();
			Writer << 0;
			Writer << false;
			Writer.EndArray();
			Writer.BeginArray();
			Writer.AddObjectAttachment(MakeTestHash(10));
			Writer << false;
			Writer.EndArray();
		}
		Writer.EndArray();
		// Uniform array of uniform objects.
		Writer.BeginArray();
		{
			Writer.BeginObject();
			Writer.SetName("ObjAttachmentInUniObjInUniObj1").AddObjectAttachment(MakeTestHash(11));
			Writer.SetName("ObjAttachmentInUniObjInUniObj2").AddObjectAttachment(MakeTestHash(12));
			Writer.EndObject();
			Writer.BeginObject();
			Writer.SetName("ObjAttachmentInUniObjInUniObj3").AddObjectAttachment(MakeTestHash(13));
			Writer.SetName("ObjAttachmentInUniObjInUniObj4").AddObjectAttachment(MakeTestHash(14));
			Writer.EndObject();
		}
		Writer.EndArray();
		// Uniform array of non-uniform objects.
		Writer.BeginArray();
		{
			Writer.BeginObject();
			Writer << "Int" << 0;
			Writer << "Bool" << false;
			Writer.EndObject();
			Writer.BeginObject();
			Writer.SetName("ObjAttachmentInNonUniObjInUniObj").AddObjectAttachment(MakeTestHash(15));
			Writer << "Bool" << false;
			Writer.EndObject();
		}
		Writer.EndArray();

		// Uniform object of type to ignore.
		Writer.BeginObject();
		{
			Writer << "Int1" << 1;
			Writer << "Int2" << 2;
		}
		Writer.EndObject();
		// Uniform object of binary attachments.
		Writer.BeginObject();
		{
			Writer.SetName("BinAttachmentInUniObj1").AddBinaryAttachment(MakeTestHash(16));
			Writer.SetName("BinAttachmentInUniObj2").AddBinaryAttachment(MakeTestHash(17));
		}
		Writer.EndObject();
		// Uniform object of uniform arrays.
		Writer.BeginObject();
		{
			Writer.SetName("Array1");
			Writer.BeginArray();
			Writer.AddBinaryAttachment(MakeTestHash(18));
			Writer.AddBinaryAttachment(MakeTestHash(19));
			Writer.EndArray();
			Writer.SetName("Array2");
			Writer.BeginArray();
			Writer.AddBinaryAttachment(MakeTestHash(20));
			Writer.AddBinaryAttachment(MakeTestHash(21));
			Writer.EndArray();
		}
		Writer.EndObject();
		// Uniform object of non-uniform arrays.
		Writer.BeginObject();
		{
			Writer.SetName("Array1");
			Writer.BeginArray();
			Writer << 0;
			Writer << false;
			Writer.EndArray();
			Writer.SetName("Array2");
			Writer.BeginArray();
			Writer.AddObjectAttachment(MakeTestHash(22));
			Writer << false;
			Writer.EndArray();
		}
		Writer.EndObject();
		// Uniform object of uniform objects.
		Writer.BeginObject();
		{
			Writer.SetName("Object1");
			Writer.BeginObject();
			Writer.SetName("ObjAttachmentInUniObjInUniObj1").AddObjectAttachment(MakeTestHash(23));
			Writer.SetName("ObjAttachmentInUniObjInUniObj2").AddObjectAttachment(MakeTestHash(24));
			Writer.EndObject();
			Writer.SetName("Object2");
			Writer.BeginObject();
			Writer.SetName("ObjAttachmentInUniObjInUniObj3").AddObjectAttachment(MakeTestHash(25));
			Writer.SetName("ObjAttachmentInUniObjInUniObj4").AddObjectAttachment(MakeTestHash(26));
			Writer.EndObject();
		}
		Writer.EndObject();
		// Uniform object of non-uniform objects.
		Writer.BeginObject();
		{
			Writer.SetName("Object1");
			Writer.BeginObject();
			Writer << "Int" << 0;
			Writer << "Bool" << false;
			Writer.EndObject();
			Writer.SetName("Object2");
			Writer.BeginObject();
			Writer.SetName("ObjAttachmentInNonUniObjInUniObj").AddObjectAttachment(MakeTestHash(27));
			Writer << "Bool" << false;
			Writer.EndObject();
		}
		Writer.EndObject();

		Fields = Writer.Save();
	}

	TestEqual(TEXT("FCbFieldView::IterateAttachments Validate"), ValidateCompactBinaryRange(Fields.GetOuterBuffer(), ECbValidateMode::All), ECbValidateError::None);

	uint32 AttachmentIndex = 0;
	Fields.IterateRangeAttachments([this, &AttachmentIndex, &MakeTestHash](FCbFieldView Field)
		{
			TestTrue(FString::Printf(TEXT("FCbFieldView::IterateAttachments(%u)::IsAttachment"), AttachmentIndex), Field.IsAttachment());
			TestEqual(FString::Printf(TEXT("FCbFieldView::IterateAttachments(%u)"), AttachmentIndex), Field.AsAttachment(), MakeTestHash(AttachmentIndex));
			++AttachmentIndex;
		});
	TestEqual(TEXT("FCbFieldView::IterateAttachments"), AttachmentIndex, 28);

	return true;
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FCbFieldBufferTest, FCbFieldTestBase, "System.Core.Serialization.CbFieldBuffer", CompactBinaryTestFlags)
bool FCbFieldBufferTest::RunTest(const FString& Parameters)
{
	static_assert(std::is_constructible<FCbField>::value, "Missing constructor for FCbField");
	static_assert(std::is_constructible<FCbField, const FCbField&>::value, "Missing constructor for FCbField");
	static_assert(std::is_constructible<FCbField, FCbField&&>::value, "Missing constructor for FCbField");

	static_assert(std::is_constructible<FCbField, const FSharedBuffer&>::value, "Missing constructor for FCbField");
	static_assert(std::is_constructible<FCbField, FSharedBuffer&&>::value, "Missing constructor for FCbField");

	static_assert(std::is_constructible<FCbField, const FCbFieldView&, const FSharedBuffer&>::value, "Missing constructor for FCbField");
	static_assert(std::is_constructible<FCbField, const FCbFieldView&, const FCbFieldIterator&>::value, "Missing constructor for FCbField");
	static_assert(std::is_constructible<FCbField, const FCbFieldView&, const FCbField&>::value, "Missing constructor for FCbField");
	static_assert(std::is_constructible<FCbField, const FCbFieldView&, const FCbArray&>::value, "Missing constructor for FCbField");
	static_assert(std::is_constructible<FCbField, const FCbFieldView&, const FCbObject&>::value, "Missing constructor for FCbField");

	static_assert(std::is_constructible<FCbField, const FCbFieldView&, FSharedBuffer&&>::value, "Missing constructor for FCbField");
	static_assert(std::is_constructible<FCbField, const FCbFieldView&, FCbFieldIterator&&>::value, "Missing constructor for FCbField");
	static_assert(std::is_constructible<FCbField, const FCbFieldView&, FCbField&&>::value, "Missing constructor for FCbField");
	static_assert(std::is_constructible<FCbField, const FCbFieldView&, FCbArray&&>::value, "Missing constructor for FCbField");
	static_assert(std::is_constructible<FCbField, const FCbFieldView&, FCbObject&&>::value, "Missing constructor for FCbField");

	// Test FCbField()
	{
		FCbField DefaultField;
		TestFalse(TEXT("FCbField().HasValue()"), DefaultField.HasValue());
		TestFalse(TEXT("FCbField().IsOwned()"), DefaultField.IsOwned());
		DefaultField.MakeOwned();
		TestTrue(TEXT("FCbField().MakeOwned().IsOwned()"), DefaultField.IsOwned());
	}

	// Test Field w/ Type from Shared Buffer
	{
		const uint8 Value[] = { uint8(ECbFieldType::Binary), 3, 4, 5, 6 }; // Size: 3, Data: 4/5/6
		FSharedBuffer ViewBuffer = FSharedBuffer::MakeView(MakeMemoryView(Value));
		FSharedBuffer OwnedBuffer = FSharedBuffer::Clone(ViewBuffer);

		FCbField View(ViewBuffer);
		FCbField ViewMove{FSharedBuffer(ViewBuffer)};
		FCbField ViewOuterField(ImplicitConv<FCbFieldView>(View), ViewBuffer);
		FCbField ViewOuterBuffer(ImplicitConv<FCbFieldView>(View), View);
		FCbField Owned(OwnedBuffer);
		FCbField OwnedMove{FSharedBuffer(OwnedBuffer)};
		FCbField OwnedOuterField(ImplicitConv<FCbFieldView>(Owned), OwnedBuffer);
		FCbField OwnedOuterBuffer(ImplicitConv<FCbFieldView>(Owned), Owned);

		// These lines are expected to assert when uncommented.
		//FCbField InvalidOuterBuffer(ImplicitConv<FCbFieldView>(Owned), ViewBuffer);
		//FCbField InvalidOuterBufferMove(ImplicitConv<FCbFieldView>(Owned), FSharedBuffer(ViewBuffer));

		TestEqual(TEXT("FCbField(ViewBuffer)"), View.AsBinaryView(), ViewBuffer.GetView().Right(3));
		TestEqual(TEXT("FCbField(ViewBuffer&&)"), ViewMove.AsBinaryView(), View.AsBinaryView());
		TestEqual(TEXT("FCbField(ViewOuterField)"), ViewOuterField.AsBinaryView(), View.AsBinaryView());
		TestEqual(TEXT("FCbField(ViewOuterBuffer)"), ViewOuterBuffer.AsBinaryView(), View.AsBinaryView());
		TestEqual(TEXT("FCbField(OwnedBuffer)"), Owned.AsBinaryView(), OwnedBuffer.GetView().Right(3));
		TestEqual(TEXT("FCbField(OwnedBuffer&&)"), OwnedMove.AsBinaryView(), Owned.AsBinaryView());
		TestEqual(TEXT("FCbField(OwnedOuterField)"), OwnedOuterField.AsBinaryView(), Owned.AsBinaryView());
		TestEqual(TEXT("FCbField(OwnedOuterBuffer)"), OwnedOuterBuffer.AsBinaryView(), Owned.AsBinaryView());

		TestFalse(TEXT("FCbField(ViewBuffer).IsOwned()"), View.IsOwned());
		TestFalse(TEXT("FCbField(ViewBuffer&&).IsOwned()"), ViewMove.IsOwned());
		TestFalse(TEXT("FCbField(ViewOuterField).IsOwned()"), ViewOuterField.IsOwned());
		TestFalse(TEXT("FCbField(ViewOuterBuffer).IsOwned()"), ViewOuterBuffer.IsOwned());
		TestTrue(TEXT("FCbField(OwnedBuffer).IsOwned()"), Owned.IsOwned());
		TestTrue(TEXT("FCbField(OwnedBuffer&&).IsOwned()"), OwnedMove.IsOwned());
		TestTrue(TEXT("FCbField(OwnedOuterField).IsOwned()"), OwnedOuterField.IsOwned());
		TestTrue(TEXT("FCbField(OwnedOuterBuffer).IsOwned()"), OwnedOuterBuffer.IsOwned());

		View.MakeOwned();
		Owned.MakeOwned();
		TestNotEqual(TEXT("FCbField(View).MakeOwned()"), View.AsBinaryView(), ViewBuffer.GetView().Right(3));
		TestTrue(TEXT("FCbField(View).MakeOwned().IsOwned()"), View.IsOwned());
		TestEqual(TEXT("FCbField(Owned).MakeOwned()"), Owned.AsBinaryView(), OwnedBuffer.GetView().Right(3));
		TestTrue(TEXT("FCbField(Owned).MakeOwned().IsOwned()"), Owned.IsOwned());
	}

	// Test Field w/ Type
	{
		const uint8 Value[] = { uint8(ECbFieldType::Binary), 3, 4, 5, 6 }; // Size: 3, Data: 4/5/6
		const FCbFieldView Field(Value);

		FCbField VoidView = FCbField::MakeView(Value);
		FCbField VoidClone = FCbField::Clone(Value);
		FCbField FieldView = FCbField::MakeView(Field);
		FCbField FieldClone = FCbField::Clone(Field);
		FCbField FieldViewClone = FCbField::Clone(FieldView);

		TestEqual(TEXT("FCbField::MakeView(Void)"), VoidView.AsBinaryView(), MakeMemoryView(Value).Right(3));
		TestNotEqual(TEXT("FCbField::Clone(Void)"), VoidClone.AsBinaryView(), MakeMemoryView(Value).Right(3));
		TestTrue(TEXT("FCbField::Clone(Void)->EqualBytes"), VoidClone.AsBinaryView().EqualBytes(VoidView.AsBinaryView()));
		TestEqual(TEXT("FCbField::MakeView(Field)"), FieldView.AsBinaryView(), MakeMemoryView(Value).Right(3));
		TestNotEqual(TEXT("FCbField::Clone(Field)"), FieldClone.AsBinaryView(), MakeMemoryView(Value).Right(3));
		TestTrue(TEXT("FCbField::Clone(Field)->EqualBytes"), FieldClone.AsBinaryView().EqualBytes(VoidView.AsBinaryView()));
		TestNotEqual(TEXT("FCbField::Clone(FieldView)"), FieldViewClone.AsBinaryView(), FieldView.AsBinaryView());
		TestTrue(TEXT("FCbField::Clone(FieldView)->EqualBytes"), FieldViewClone.AsBinaryView().EqualBytes(VoidView.AsBinaryView()));

		TestFalse(TEXT("FCbField::MakeView(Void).IsOwned()"), VoidView.IsOwned());
		TestTrue(TEXT("FCbField::Clone(Void).IsOwned()"), VoidClone.IsOwned());
		TestFalse(TEXT("FCbField::MakeView(Field).IsOwned()"), FieldView.IsOwned());
		TestTrue(TEXT("FCbField::Clone(Field).IsOwned()"), FieldClone.IsOwned());
		TestTrue(TEXT("FCbField::Clone(FieldView).IsOwned()"), FieldViewClone.IsOwned());
	}

	// Test Field w/o Type
	{
		const uint8 Value[] = { 3, 4, 5, 6 }; // Size: 3, Data: 4/5/6
		const FCbFieldView Field(Value, ECbFieldType::Binary);

		FCbField FieldView = FCbField::MakeView(Field);
		FCbField FieldClone = FCbField::Clone(Field);
		FCbField FieldViewClone = FCbField::Clone(FieldView);

		TestEqual(TEXT("FCbField::MakeView(Field, NoType)"), FieldView.AsBinaryView(), MakeMemoryView(Value).Right(3));
		TestTrue(TEXT("FCbField::Clone(Field, NoType)"), FieldClone.AsBinaryView().EqualBytes(FieldView.AsBinaryView()));
		TestTrue(TEXT("FCbField::Clone(FieldView, NoType)"), FieldViewClone.AsBinaryView().EqualBytes(FieldView.AsBinaryView()));

		TestFalse(TEXT("FCbField::MakeView(Field, NoType).IsOwned()"), FieldView.IsOwned());
		TestTrue(TEXT("FCbField::Clone(Field, NoType).IsOwned()"), FieldClone.IsOwned());
		TestTrue(TEXT("FCbField::Clone(FieldView, NoType).IsOwned()"), FieldViewClone.IsOwned());

		FieldView.MakeOwned();
		TestTrue(TEXT("FCbField::MakeView(NoType).MakeOwned()"), FieldView.AsBinaryView().EqualBytes(MakeMemoryView(Value).Right(3)));
		TestTrue(TEXT("FCbField::MakeView(NoType).MakeOwned().IsOwned()"), FieldView.IsOwned());
	}

	return true;
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FCbArrayBufferTest, "System.Core.Serialization.CbArrayBuffer", CompactBinaryTestFlags)
bool FCbArrayBufferTest::RunTest(const FString& Parameters)
{
	static_assert(std::is_constructible<FCbArray>::value, "Missing constructor for FCbArray");
	static_assert(std::is_constructible<FCbArray, const FCbArray&>::value, "Missing constructor for FCbArray");
	static_assert(std::is_constructible<FCbArray, FCbArray&&>::value, "Missing constructor for FCbArray");

	static_assert(std::is_constructible<FCbArray, const FCbArrayView&, const FSharedBuffer&>::value, "Missing constructor for FCbArray");
	static_assert(std::is_constructible<FCbArray, const FCbArrayView&, const FCbFieldIterator&>::value, "Missing constructor for FCbArray");
	static_assert(std::is_constructible<FCbArray, const FCbArrayView&, const FCbField&>::value, "Missing constructor for FCbArray");
	static_assert(std::is_constructible<FCbArray, const FCbArrayView&, const FCbArray&>::value, "Missing constructor for FCbArray");
	static_assert(std::is_constructible<FCbArray, const FCbArrayView&, const FCbObject&>::value, "Missing constructor for FCbArray");

	static_assert(std::is_constructible<FCbArray, const FCbArrayView&, FSharedBuffer&&>::value, "Missing constructor for FCbArray");
	static_assert(std::is_constructible<FCbArray, const FCbArrayView&, FCbFieldIterator&&>::value, "Missing constructor for FCbArray");
	static_assert(std::is_constructible<FCbArray, const FCbArrayView&, FCbField&&>::value, "Missing constructor for FCbArray");
	static_assert(std::is_constructible<FCbArray, const FCbArrayView&, FCbArray&&>::value, "Missing constructor for FCbArray");
	static_assert(std::is_constructible<FCbArray, const FCbArrayView&, FCbObject&&>::value, "Missing constructor for FCbArray");

	// Test FCbArray()
	{
		FCbArray DefaultArray;
		TestFalse(TEXT("FCbArray().IsOwned()"), DefaultArray.IsOwned());
		DefaultArray.MakeOwned();
		TestTrue(TEXT("FCbArray().MakeOwned().IsOwned()"), DefaultArray.IsOwned());
	}

	return true;
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FCbObjectBufferTest, "System.Core.Serialization.CbObjectBuffer", CompactBinaryTestFlags)
bool FCbObjectBufferTest::RunTest(const FString& Parameters)
{
	static_assert(std::is_constructible<FCbObject>::value, "Missing constructor for FCbObject");
	static_assert(std::is_constructible<FCbObject, const FCbObject&&>::value, "Missing constructor for FCbObject");
	static_assert(std::is_constructible<FCbObject, FCbObject&&>::value, "Missing constructor for FCbObject");

	static_assert(std::is_constructible<FCbObject, const FCbObjectView&, const FSharedBuffer&>::value, "Missing constructor for FCbObject");
	static_assert(std::is_constructible<FCbObject, const FCbObjectView&, const FCbFieldIterator&>::value, "Missing constructor for FCbObject");
	static_assert(std::is_constructible<FCbObject, const FCbObjectView&, const FCbField&>::value, "Missing constructor for FCbObject");
	static_assert(std::is_constructible<FCbObject, const FCbObjectView&, const FCbArray&>::value, "Missing constructor for FCbObject");
	static_assert(std::is_constructible<FCbObject, const FCbObjectView&, const FCbObject&>::value, "Missing constructor for FCbObject");

	static_assert(std::is_constructible<FCbObject, const FCbObjectView&, FSharedBuffer&&>::value, "Missing constructor for FCbObject");
	static_assert(std::is_constructible<FCbObject, const FCbObjectView&, FCbFieldIterator&&>::value, "Missing constructor for FCbObject");
	static_assert(std::is_constructible<FCbObject, const FCbObjectView&, FCbField&&>::value, "Missing constructor for FCbObject");
	static_assert(std::is_constructible<FCbObject, const FCbObjectView&, FCbArray&&>::value, "Missing constructor for FCbObject");
	static_assert(std::is_constructible<FCbObject, const FCbObjectView&, FCbObject&&>::value, "Missing constructor for FCbObject");

	// Test FCbObject()
	{
		FCbObject DefaultObject;
		TestFalse(TEXT("FCbObject().IsOwned()"), DefaultObject.IsOwned());
		DefaultObject.MakeOwned();
		TestTrue(TEXT("FCbObject().MakeOwned().IsOwned()"), DefaultObject.IsOwned());
	}

	return true;
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FCbFieldBufferIteratorTest, "System.Core.Serialization.CbFieldBufferIterator", CompactBinaryTestFlags)
bool FCbFieldBufferIteratorTest::RunTest(const FString& Parameters)
{
	static_assert(std::is_constructible<FCbFieldViewIterator, const FCbFieldIterator&>::value, "Missing constructor for FCbFieldViewIterator");
	static_assert(std::is_constructible<FCbFieldViewIterator, FCbFieldIterator&&>::value, "Missing constructor for FCbFieldViewIterator");

	static_assert(std::is_constructible<FCbFieldIterator, const FCbFieldIterator&>::value, "Missing constructor for FCbFieldIterator");
	static_assert(std::is_constructible<FCbFieldIterator, FCbFieldIterator&&>::value, "Missing constructor for FCbFieldIterator");

	const auto GetCount = [](auto It) -> uint32
	{
		uint32 Count = 0;
		for (; It; ++It)
		{
			++Count;
		}
		return Count;
	};

	// Test FCbField[View]Iterator()
	{
		TestEqual(TEXT("FCbFieldViewIterator()"), GetCount(FCbFieldViewIterator()), 0);
		TestEqual(TEXT("FCbFieldIterator()"), GetCount(FCbFieldIterator()), 0);
	}

	// Test FCbField[View]Iterator(Range)
	{
		constexpr uint8 T = uint8(ECbFieldType::IntegerPositive);
		const uint8 Value[] = { T, 0, T, 1, T, 2, T, 3 };

		const FSharedBuffer View = FSharedBuffer::MakeView(MakeMemoryView(Value));
		const FSharedBuffer Clone = FSharedBuffer::Clone(View);

		const FMemoryView EmptyView;
		const FSharedBuffer NullBuffer;

		const FCbFieldViewIterator FieldViewIt = FCbFieldViewIterator::MakeRange(View);
		const FCbFieldIterator FieldIt = FCbFieldIterator::MakeRange(View);

		TestEqual(TEXT("FCbFieldViewIterator::GetRangeHash()"), FieldViewIt.GetRangeHash(), FIoHash::HashBuffer(View));
		TestEqual(TEXT("FCbFieldIterator::GetRangeHash()"), FieldIt.GetRangeHash(), FIoHash::HashBuffer(View));

		FMemoryView RangeView;
		TestTrue(TEXT("FCbFieldViewIterator::TryGetRangeView()"), FieldViewIt.TryGetRangeView(RangeView) && RangeView == MakeMemoryView(Value));
		TestTrue(TEXT("FCbFieldIterator::TryGetRangeView()"), FieldIt.TryGetRangeView(RangeView) && RangeView == MakeMemoryView(Value));

		TestEqual(TEXT("FCbFieldIterator::CloneRange(EmptyViewIt)"), GetCount(FCbFieldIterator::CloneRange(FCbFieldViewIterator())), 0);
		TestEqual(TEXT("FCbFieldIterator::CloneRange(EmptyIt)"), GetCount(FCbFieldIterator::CloneRange(FCbFieldIterator())), 0);
		const FCbFieldIterator FieldViewItClone = FCbFieldIterator::CloneRange(FieldViewIt);
		const FCbFieldIterator FieldItClone = FCbFieldIterator::CloneRange(FieldIt);
		TestEqual(TEXT("FCbFieldIterator::CloneRange(FieldViewIt)"), GetCount(FieldViewItClone), 4);
		TestEqual(TEXT("FCbFieldIterator::CloneRange(FieldIt)"), GetCount(FieldItClone), 4);
		TestNotEqual(TEXT("FCbFieldIterator::CloneRange(FieldViewIt).Equals()"), FieldViewItClone, FieldIt);
		TestNotEqual(TEXT("FCbFieldIterator::CloneRange(FieldIt).Equals()"), FieldItClone, FieldIt);

		TestEqual(TEXT("FCbFieldViewIterator::MakeRange(EmptyView)"), GetCount(FCbFieldViewIterator::MakeRange(EmptyView)), 0);
		TestEqual(TEXT("FCbFieldIterator::MakeRange(BufferNullL)"), GetCount(FCbFieldIterator::MakeRange(NullBuffer)), 0);
		TestEqual(TEXT("FCbFieldIterator::MakeRange(BufferNullR)"), GetCount(FCbFieldIterator::MakeRange(FSharedBuffer(NullBuffer))), 0);

		TestEqual(TEXT("FCbFieldViewIterator::MakeRange(BufferView)"), GetCount(FCbFieldViewIterator::MakeRange(MakeMemoryView(Value))), 4);
		TestEqual(TEXT("FCbFieldIterator::MakeRange(BufferCloneL)"), GetCount(FCbFieldIterator::MakeRange(Clone)), 4);
		TestEqual(TEXT("FCbFieldIterator::MakeRange(BufferCloneR)"), GetCount(FCbFieldIterator::MakeRange(FSharedBuffer(Clone))), 4);

		TestEqual(TEXT("FCbFieldIterator::MakeRangeView(FieldIt, BufferNullL)"), GetCount(FCbFieldIterator::MakeRangeView(FCbFieldViewIterator::MakeRange(View), NullBuffer)), 4);
		TestEqual(TEXT("FCbFieldIterator::MakeRangeView(FieldIt, BufferNullR)"), GetCount(FCbFieldIterator::MakeRangeView(FCbFieldViewIterator::MakeRange(View), FSharedBuffer(NullBuffer))), 4);
		TestEqual(TEXT("FCbFieldIterator::MakeRangeView(FieldIt, BufferViewL)"), GetCount(FCbFieldIterator::MakeRangeView(FCbFieldViewIterator::MakeRange(View), View)), 4);
		TestEqual(TEXT("FCbFieldIterator::MakeRangeView(FieldIt, BufferViewR)"), GetCount(FCbFieldIterator::MakeRangeView(FCbFieldViewIterator::MakeRange(View), FSharedBuffer(View))), 4);
		TestEqual(TEXT("FCbFieldIterator::MakeRangeView(FieldIt, BufferCloneL)"), GetCount(FCbFieldIterator::MakeRangeView(FCbFieldViewIterator::MakeRange(Clone), Clone)), 4);
		TestEqual(TEXT("FCbFieldIterator::MakeRangeView(FieldIt, BufferCloneR)"), GetCount(FCbFieldIterator::MakeRangeView(FCbFieldViewIterator::MakeRange(Clone), FSharedBuffer(Clone))), 4);

		TestEqual(TEXT("FCbFieldViewIterator(FieldItL)"), GetCount(FCbFieldViewIterator(FieldIt)), 4);
		TestEqual(TEXT("FCbFieldViewIterator(FieldItR)"), GetCount(FCbFieldViewIterator(FCbFieldIterator(FieldIt))), 4);

		// Uniform
		const uint8 UniformValue[] = { 0, 1, 2, 3 };
		const FCbFieldViewIterator UniformFieldViewIt = FCbFieldViewIterator::MakeRange(MakeMemoryView(UniformValue), ECbFieldType::IntegerPositive);
		TestEqual(TEXT("FCbFieldViewIterator::MakeRange(Uniform).GetRangeHash()"), UniformFieldViewIt.GetRangeHash(), FieldViewIt.GetRangeHash());
		TestFalse(TEXT("FCbFieldViewIterator::MakeRange(Uniform).TryGetRangeView()"), UniformFieldViewIt.TryGetRangeView(RangeView));
		const FSharedBuffer UniformView = FSharedBuffer::MakeView(MakeMemoryView(UniformValue));
		const FCbFieldIterator UniformFieldIt = FCbFieldIterator::MakeRange(UniformView, ECbFieldType::IntegerPositive);
		TestEqual(TEXT("FCbFieldIterator::MakeRange(Uniform).GetRangeHash()"), UniformFieldIt.GetRangeHash(), FieldViewIt.GetRangeHash());
		TestFalse(TEXT("FCbFieldIterator::MakeRange(Uniform).TryGetRangeView()"), UniformFieldIt.TryGetRangeView(RangeView));

		// Equals
		TestTrue(TEXT("FCbFieldViewIterator::Equals(Self)"), FieldViewIt.Equals(AsConst(FieldViewIt)));
		TestTrue(TEXT("FCbFieldViewIterator::Equals(OtherType)"), FieldViewIt.Equals(AsConst(FieldIt)));
		TestTrue(TEXT("FCbFieldIterator::Equals(Self)"), FieldIt.Equals(AsConst(FieldIt)));
		TestTrue(TEXT("FCbFieldIterator::Equals(OtherType)"), FieldIt.Equals(AsConst(FieldViewIt)));
		TestFalse(TEXT("FCbFieldViewIterator::Equals(OtherRange)"), FieldViewIt.Equals(FieldViewItClone));
		TestFalse(TEXT("FCbFieldIterator::Equals(OtherRange)"), FieldIt.Equals(FieldItClone));
		TestTrue(TEXT("FCbFieldViewIterator::Equals(Uniform, Self)"), UniformFieldViewIt.Equals(AsConst(UniformFieldViewIt)));
		TestTrue(TEXT("FCbFieldViewIterator::Equals(Uniform, OtherType)"), UniformFieldViewIt.Equals(UniformFieldIt));
		TestTrue(TEXT("FCbFieldIterator::Equals(Uniform, Self)"), UniformFieldIt.Equals(AsConst(UniformFieldIt)));
		TestTrue(TEXT("FCbFieldIterator::Equals(Uniform, OtherType)"), UniformFieldIt.Equals(UniformFieldViewIt));
		TestFalse(TEXT("FCbFieldViewIterator::Equals(SameValue, DifferentEnd)"),
			FCbFieldViewIterator::MakeRange(MakeMemoryView(UniformValue), ECbFieldType::IntegerPositive)
				.Equals(FCbFieldViewIterator::MakeRange(MakeMemoryView(UniformValue).LeftChop(1), ECbFieldType::IntegerPositive)));
		TestFalse(TEXT("FCbFieldViewIterator::Equals(DifferentValue, SameEnd)"),
			FCbFieldViewIterator::MakeRange(MakeMemoryView(UniformValue), ECbFieldType::IntegerPositive)
				.Equals(FCbFieldViewIterator::MakeRange(MakeMemoryView(UniformValue).RightChop(1), ECbFieldType::IntegerPositive)));

		// CopyRangeTo
		uint8 CopyBytes[sizeof(Value)];
		FieldViewIt.CopyRangeTo(MakeMemoryView(CopyBytes));
		TestTrue(TEXT("FCbFieldViewIterator::MakeRange().CopyRangeTo()"), MakeMemoryView(CopyBytes).EqualBytes(MakeMemoryView(Value)));
		FieldIt.CopyRangeTo(MakeMemoryView(CopyBytes));
		TestTrue(TEXT("FCbFieldIterator::MakeRange().CopyRangeTo()"), MakeMemoryView(CopyBytes).EqualBytes(MakeMemoryView(Value)));
		UniformFieldViewIt.CopyRangeTo(MakeMemoryView(CopyBytes));
		TestTrue(TEXT("FCbFieldViewIterator::MakeRange(Uniform).CopyRangeTo()"), MakeMemoryView(CopyBytes).EqualBytes(MakeMemoryView(Value)));

		// MakeRangeOwned
		FCbFieldIterator OwnedFromView = UniformFieldIt;
		OwnedFromView.MakeRangeOwned();
		TestTrue(TEXT("FCbFieldIterator::MakeRangeOwned(View)"), OwnedFromView.TryGetRangeView(RangeView) && RangeView.EqualBytes(MakeMemoryView(Value)));
		FCbFieldIterator OwnedFromOwned = OwnedFromView;
		OwnedFromOwned.MakeRangeOwned();
		TestEqual(TEXT("FCbFieldIterator::MakeRangeOwned(Owned)"), OwnedFromOwned, OwnedFromView);

		// These lines are expected to assert when uncommented.
		//const FSharedBuffer ShortView = FSharedBuffer::MakeView(MakeMemoryView(Value).LeftChop(2));
		//TestEqual(TEXT("FCbFieldIterator::MakeRangeView(FieldIt, InvalidBufferL)"), GetCount(FCbFieldIterator::MakeRangeView(FCbFieldViewIterator::MakeRange(*View), ShortView)), 4);
		//TestEqual(TEXT("FCbFieldIterator::MakeRangeView(FieldIt, InvalidBufferR)"), GetCount(FCbFieldIterator::MakeRangeView(FCbFieldViewIterator::MakeRange(*View), FSharedBuffer(ShortView))), 4);
	}

	// Test FCbField[View]Iterator(Scalar)
	{
		constexpr uint8 T = uint8(ECbFieldType::IntegerPositive);
		const uint8 Value[] = { T, 0 };

		const FSharedBuffer View = FSharedBuffer::MakeView(MakeMemoryView(Value));
		const FSharedBuffer Clone = FSharedBuffer::Clone(View);

		const FCbFieldView FieldView(Value);
		const FCbField Field(View);

		TestEqual(TEXT("FCbFieldViewIterator::MakeSingle(FieldViewL)"), GetCount(FCbFieldViewIterator::MakeSingle(FieldView)), 1);
		TestEqual(TEXT("FCbFieldViewIterator::MakeSingle(FieldViewR)"), GetCount(FCbFieldViewIterator::MakeSingle(FCbFieldView(FieldView))), 1);
		TestEqual(TEXT("FCbFieldIterator::MakeSingle(FieldL)"), GetCount(FCbFieldIterator::MakeSingle(Field)), 1);
		TestEqual(TEXT("FCbFieldIterator::MakeSingle(FieldR)"), GetCount(FCbFieldIterator::MakeSingle(FCbField(Field))), 1);
	}

	return true;
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FCbFieldParseTest, "System.Core.Serialization.CbFieldParseTest", CompactBinaryTestFlags)
bool FCbFieldParseTest::RunTest(const FString& Parameters)
{
	// Test the optimal object parsing loop because it is expected to be required for high performance.
	// Under ideal conditions, when the fields are in the expected order and there are no extra fields,
	// the loop will execute once and only one comparison will be performed for each field name. Either
	// way, each field will only be visited once even if the loop needs to execute several times.
	auto ParseObject = [this](const FCbObjectView& Object, uint32& A, uint32& B, uint32& C, uint32& D)
	{
		for (FCbFieldViewIterator It = Object.CreateViewIterator(); It;)
		{
			const FCbFieldViewIterator Last = It;
			if (It.GetName().Equals(UTF8TEXTVIEW("A")))
			{
				A = It.AsUInt32();
				++It;
			}
			if (It.GetName().Equals(UTF8TEXTVIEW("B")))
			{
				B = It.AsUInt32();
				++It;
			}
			if (It.GetName().Equals(UTF8TEXTVIEW("C")))
			{
				C = It.AsUInt32();
				++It;
			}
			if (It.GetName().Equals(UTF8TEXTVIEW("D")))
			{
				D = It.AsUInt32();
				++It;
			}
			if (Last == It)
			{
				++It;
			}
		}
	};

	auto TestParseObject = [this, &ParseObject](std::initializer_list<uint8> Data, uint32 A, uint32 B, uint32 C, uint32 D) -> bool
	{
		uint32 ParsedA = 0, ParsedB = 0, ParsedC = 0, ParsedD = 0;
		ParseObject(FCbObjectView(GetData(Data), ECbFieldType::Object), ParsedA, ParsedB, ParsedC, ParsedD);
		return A == ParsedA && B == ParsedB && C == ParsedC && D == ParsedD;
	};

	constexpr uint8 T = uint8(ECbFieldType::IntegerPositive | ECbFieldType::HasFieldName);
	TestTrue(TEXT("FCbObjectView Parse(None)"), TestParseObject({0}, 0, 0, 0, 0));
	TestTrue(TEXT("FCbObjectView Parse(ABCD)"), TestParseObject({16, T, 1, 'A', 1, T, 1, 'B', 2, T, 1, 'C', 3, T, 1, 'D', 4}, 1, 2, 3, 4));
	TestTrue(TEXT("FCbObjectView Parse(BCDA)"), TestParseObject({16, T, 1, 'B', 2, T, 1, 'C', 3, T, 1, 'D', 4, T, 1, 'A', 1}, 1, 2, 3, 4));
	TestTrue(TEXT("FCbObjectView Parse(BCD)"), TestParseObject({12, T, 1, 'B', 2, T, 1, 'C', 3, T, 1, 'D', 4}, 0, 2, 3, 4));
	TestTrue(TEXT("FCbObjectView Parse(BC)"), TestParseObject({8, T, 1, 'B', 2, T, 1, 'C', 3}, 0, 2, 3, 0));
	TestTrue(TEXT("FCbObjectView Parse(ABCDE)"), TestParseObject({20, T, 1, 'A', 1, T, 1, 'B', 2, T, 1, 'C', 3, T, 1, 'D', 4, T, 1, 'E', 5}, 1, 2, 3, 4));
	TestTrue(TEXT("FCbObjectView Parse(EABCD)"), TestParseObject({20, T, 1, 'E', 5, T, 1, 'A', 1, T, 1, 'B', 2, T, 1, 'C', 3, T, 1, 'D', 4}, 1, 2, 3, 4));
	TestTrue(TEXT("FCbObjectView Parse(DCBA)"), TestParseObject({16, T, 1, 'D', 4, T, 1, 'C', 3, T, 1, 'B', 2, T, 1, 'A', 1}, 1, 2, 3, 4));

	return true;
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

#endif // WITH_DEV_AUTOMATION_TESTS
