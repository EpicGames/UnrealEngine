// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/WarnFilterScope.h"
#include "Logging/StructuredLog.h"
#include "Misc/FeedbackContext.h"
#include "Misc/StringBuilder.h"

struct FFilterFeedback : public FFeedbackContext
{

	FFilterFeedback(FFeedbackContext* OldFeedback, TUniqueFunction<bool(const TCHAR* Message, ELogVerbosity::Type Verbosity, const FName& Category)> LogHandler)
		: OldFeedback(OldFeedback)
		, Handler(MoveTemp(LogHandler))
	{
	}

	virtual void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const FName& Category) override
	{
		if (!Handler(V, Verbosity, Category))
		{
			OldFeedback->Serialize(V, Verbosity, Category);
		}
	}

	virtual void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const FName& Category, double Time) override
	{
		if (!Handler(V, Verbosity, Category))
		{
			OldFeedback->Serialize(V, Verbosity, Category, Time);
		}
	}

	virtual void SerializeRecord(const UE::FLogRecord& Record) override
	{
		TStringBuilder<256> Message;
		Record.FormatMessageTo(Message);
		if (!Handler(*Message, Record.GetVerbosity(), Record.GetCategory()))
		{
			OldFeedback->SerializeRecord(Record);
		}
	}

	FFeedbackContext* OldFeedback;
	TUniqueFunction<bool(const TCHAR* Message, ELogVerbosity::Type Verbosity, const FName& Category)> Handler;
};

FWarnFilterScope::FWarnFilterScope(TUniqueFunction<bool(const TCHAR* Message, ELogVerbosity::Type Verbosity, const FName& Category)> LogHandler)
	: OldWarn(GWarn)
	, Feedback(new FFilterFeedback(OldWarn, MoveTemp(LogHandler)))
{
	GWarn = Feedback;
}

FWarnFilterScope::~FWarnFilterScope()
{
	GWarn = OldWarn;
	delete Feedback;
}
