// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "Containers/ContainersFwd.h"
#include "Misc/OptionalFwd.h"
#include "Templates/FunctionFwd.h"
#include "Templates/TupleFwd.h"
#include "Traits/FunctionArity.h"

#if WITH_TESTS

namespace
{
	// 0-arity
	static_assert(TFunctionArity_V<void()                 > == 0);
	static_assert(TFunctionArity_V<bool()                 > == 0);
	static_assert(TFunctionArity_V<void() const           > == 0);
	static_assert(TFunctionArity_V<bool() const           > == 0);
	static_assert(TFunctionArity_V<void()       volatile  > == 0);
	static_assert(TFunctionArity_V<bool()       volatile  > == 0);
	static_assert(TFunctionArity_V<void() const volatile  > == 0);
	static_assert(TFunctionArity_V<bool() const volatile  > == 0);
	static_assert(TFunctionArity_V<void()               & > == 0);
	static_assert(TFunctionArity_V<bool()               & > == 0);
	static_assert(TFunctionArity_V<void() const         & > == 0);
	static_assert(TFunctionArity_V<bool() const         & > == 0);
	static_assert(TFunctionArity_V<void()       volatile& > == 0);
	static_assert(TFunctionArity_V<bool()       volatile& > == 0);
	static_assert(TFunctionArity_V<void() const volatile& > == 0);
	static_assert(TFunctionArity_V<bool() const volatile& > == 0);
	static_assert(TFunctionArity_V<void()               &&> == 0);
	static_assert(TFunctionArity_V<bool()               &&> == 0);
	static_assert(TFunctionArity_V<void() const         &&> == 0);
	static_assert(TFunctionArity_V<bool() const         &&> == 0);
	static_assert(TFunctionArity_V<void()       volatile&&> == 0);
	static_assert(TFunctionArity_V<bool()       volatile&&> == 0);
	static_assert(TFunctionArity_V<void() const volatile&&> == 0);
	static_assert(TFunctionArity_V<bool() const volatile&&> == 0);

	// 1-arity
	static_assert(TFunctionArity_V<void(int)                 > == 1);
	static_assert(TFunctionArity_V<bool(int)                 > == 1);
	static_assert(TFunctionArity_V<void(int) const           > == 1);
	static_assert(TFunctionArity_V<bool(int) const           > == 1);
	static_assert(TFunctionArity_V<void(int)       volatile  > == 1);
	static_assert(TFunctionArity_V<bool(int)       volatile  > == 1);
	static_assert(TFunctionArity_V<void(int) const volatile  > == 1);
	static_assert(TFunctionArity_V<bool(int) const volatile  > == 1);
	static_assert(TFunctionArity_V<void(int)               & > == 1);
	static_assert(TFunctionArity_V<bool(int)               & > == 1);
	static_assert(TFunctionArity_V<void(int) const         & > == 1);
	static_assert(TFunctionArity_V<bool(int) const         & > == 1);
	static_assert(TFunctionArity_V<void(int)       volatile& > == 1);
	static_assert(TFunctionArity_V<bool(int)       volatile& > == 1);
	static_assert(TFunctionArity_V<void(int) const volatile& > == 1);
	static_assert(TFunctionArity_V<bool(int) const volatile& > == 1);
	static_assert(TFunctionArity_V<void(int)               &&> == 1);
	static_assert(TFunctionArity_V<bool(int)               &&> == 1);
	static_assert(TFunctionArity_V<void(int) const         &&> == 1);
	static_assert(TFunctionArity_V<bool(int) const         &&> == 1);
	static_assert(TFunctionArity_V<void(int)       volatile&&> == 1);
	static_assert(TFunctionArity_V<bool(int)       volatile&&> == 1);
	static_assert(TFunctionArity_V<void(int) const volatile&&> == 1);
	static_assert(TFunctionArity_V<bool(int) const volatile&&> == 1);

	// 2-arity
	static_assert(TFunctionArity_V<void(int, float)                 > == 2);
	static_assert(TFunctionArity_V<bool(int, float)                 > == 2);
	static_assert(TFunctionArity_V<void(int, float) const           > == 2);
	static_assert(TFunctionArity_V<bool(int, float) const           > == 2);
	static_assert(TFunctionArity_V<void(int, float)       volatile  > == 2);
	static_assert(TFunctionArity_V<bool(int, float)       volatile  > == 2);
	static_assert(TFunctionArity_V<void(int, float) const volatile  > == 2);
	static_assert(TFunctionArity_V<bool(int, float) const volatile  > == 2);
	static_assert(TFunctionArity_V<void(int, float)               & > == 2);
	static_assert(TFunctionArity_V<bool(int, float)               & > == 2);
	static_assert(TFunctionArity_V<void(int, float) const         & > == 2);
	static_assert(TFunctionArity_V<bool(int, float) const         & > == 2);
	static_assert(TFunctionArity_V<void(int, float)       volatile& > == 2);
	static_assert(TFunctionArity_V<bool(int, float)       volatile& > == 2);
	static_assert(TFunctionArity_V<void(int, float) const volatile& > == 2);
	static_assert(TFunctionArity_V<bool(int, float) const volatile& > == 2);
	static_assert(TFunctionArity_V<void(int, float)               &&> == 2);
	static_assert(TFunctionArity_V<bool(int, float)               &&> == 2);
	static_assert(TFunctionArity_V<void(int, float) const         &&> == 2);
	static_assert(TFunctionArity_V<bool(int, float) const         &&> == 2);
	static_assert(TFunctionArity_V<void(int, float)       volatile&&> == 2);
	static_assert(TFunctionArity_V<bool(int, float)       volatile&&> == 2);
	static_assert(TFunctionArity_V<void(int, float) const volatile&&> == 2);
	static_assert(TFunctionArity_V<bool(int, float) const volatile&&> == 2);

	// Very complex function type
	static_assert(TFunctionArity_V<TTuple<int, char, float>(bool, float, const char*, const TMap<int32, TArray<float>>, TOptional<TSet<char>>&&) const volatile&&> == 5);
}

#endif // WITH_TESTS
