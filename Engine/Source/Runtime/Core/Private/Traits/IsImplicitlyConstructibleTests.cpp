// Copyright Epic Games, Inc. All Rights Reserved.

#include "Traits/IsImplicitlyConstructible.h"

#if WITH_TESTS

namespace
{
	struct FEmpty
	{
	};
	static_assert(TIsImplicitlyConstructible_V<FEmpty>);
	static_assert(TIsImplicitlyConstructible_V<FEmpty, FEmpty>);

	struct FNotCopyable
	{
		FNotCopyable(const FNotCopyable&) = delete;
	};
	static_assert(!TIsImplicitlyConstructible_V<FNotCopyable>);
	static_assert(!TIsImplicitlyConstructible_V<FNotCopyable, FNotCopyable>);

	struct FNotCopyableButDefaultConstructible
	{
		FNotCopyableButDefaultConstructible() = default;
		FNotCopyableButDefaultConstructible(const FNotCopyableButDefaultConstructible&) = delete;
	};
	static_assert(TIsImplicitlyConstructible_V<FNotCopyableButDefaultConstructible>);
	static_assert(!TIsImplicitlyConstructible_V<FNotCopyableButDefaultConstructible, FNotCopyableButDefaultConstructible>);

	struct FDefaultedDefaultConstructor
	{
		FDefaultedDefaultConstructor() = default;
	};
	static_assert(TIsImplicitlyConstructible_V<FDefaultedDefaultConstructor>);
	static_assert(TIsImplicitlyConstructible_V<FDefaultedDefaultConstructor, FDefaultedDefaultConstructor>);

	struct FUserDefinedDefaultConstructor
	{
		FUserDefinedDefaultConstructor();
	};
	static_assert(TIsImplicitlyConstructible_V<FUserDefinedDefaultConstructor>);
	static_assert(TIsImplicitlyConstructible_V<FUserDefinedDefaultConstructor, FUserDefinedDefaultConstructor>);

	struct FDefaultedExplicitDefaultConstructor
	{
		explicit FDefaultedExplicitDefaultConstructor() = default;
	};
	static_assert(!TIsImplicitlyConstructible_V<FDefaultedExplicitDefaultConstructor>);
	static_assert(TIsImplicitlyConstructible_V<FDefaultedExplicitDefaultConstructor, FDefaultedExplicitDefaultConstructor>);

	struct FUserDefinedExplicitDefaultConstructor
	{
		explicit FUserDefinedExplicitDefaultConstructor();
	};
	static_assert(!TIsImplicitlyConstructible_V<FUserDefinedExplicitDefaultConstructor>);
	static_assert(TIsImplicitlyConstructible_V<FUserDefinedExplicitDefaultConstructor, FUserDefinedExplicitDefaultConstructor>);

	struct FSingleFieldAggregate
	{
		void* Ptr;
	};
	static_assert(TIsImplicitlyConstructible_V<FSingleFieldAggregate>);
	static_assert(TIsImplicitlyConstructible_V<FSingleFieldAggregate, FSingleFieldAggregate>);
	static_assert(!TIsImplicitlyConstructible_V<FSingleFieldAggregate, void*>);
	static_assert(!TIsImplicitlyConstructible_V<FSingleFieldAggregate, const void*>);

	struct FSingleRefFieldAggregate
	{
		void*& Ptr;
	};
	static_assert(!TIsImplicitlyConstructible_V<FSingleRefFieldAggregate>);
	static_assert(TIsImplicitlyConstructible_V<FSingleRefFieldAggregate, FSingleRefFieldAggregate>);
	static_assert(!TIsImplicitlyConstructible_V<FSingleRefFieldAggregate, void*>);
	static_assert(!TIsImplicitlyConstructible_V<FSingleRefFieldAggregate, const void*>);

	struct FSingleArgConstructor
	{
		FSingleArgConstructor(void*);
	};
	static_assert(!TIsImplicitlyConstructible_V<FSingleArgConstructor>);
	static_assert(TIsImplicitlyConstructible_V<FSingleArgConstructor, FSingleArgConstructor>);
	static_assert(TIsImplicitlyConstructible_V<FSingleArgConstructor, void*>);
	static_assert(!TIsImplicitlyConstructible_V<FSingleArgConstructor, const void*>);

	struct FExplicitSingleArgConstructor
	{
		explicit FExplicitSingleArgConstructor(void*);
	};
	static_assert(!TIsImplicitlyConstructible_V<FExplicitSingleArgConstructor>);
	static_assert(TIsImplicitlyConstructible_V<FExplicitSingleArgConstructor, FExplicitSingleArgConstructor>);
	static_assert(!TIsImplicitlyConstructible_V<FExplicitSingleArgConstructor, void*>);
	static_assert(!TIsImplicitlyConstructible_V<FExplicitSingleArgConstructor, const void*>);

	struct FTwoFieldAggregate
	{
		void* Ptr;
		bool bFlag;
	};
	static_assert(TIsImplicitlyConstructible_V<FTwoFieldAggregate>);
	static_assert(TIsImplicitlyConstructible_V<FTwoFieldAggregate, FTwoFieldAggregate>);
	static_assert(!TIsImplicitlyConstructible_V<FTwoFieldAggregate, void*>);
	static_assert(!TIsImplicitlyConstructible_V<FTwoFieldAggregate, const void*>);
	static_assert(!TIsImplicitlyConstructible_V<FTwoFieldAggregate, void*, bool>);
	static_assert(!TIsImplicitlyConstructible_V<FTwoFieldAggregate, const void*, bool>);

	struct FTwoRefFieldAggregate
	{
		void*& Ptr;
		bool& bFlag;
	};
	static_assert(!TIsImplicitlyConstructible_V<FTwoRefFieldAggregate>);
	static_assert(TIsImplicitlyConstructible_V<FTwoRefFieldAggregate, FTwoRefFieldAggregate>);
	static_assert(!TIsImplicitlyConstructible_V<FTwoRefFieldAggregate, void*>);
	static_assert(!TIsImplicitlyConstructible_V<FTwoRefFieldAggregate, const void*>);
	static_assert(!TIsImplicitlyConstructible_V<FTwoRefFieldAggregate, void*, bool>);
	static_assert(!TIsImplicitlyConstructible_V<FTwoRefFieldAggregate, const void*, bool>);

	struct FTwoArgConstructor
	{
		FTwoArgConstructor(void*, bool);
	};
	static_assert(!TIsImplicitlyConstructible_V<FTwoArgConstructor>);
	static_assert(TIsImplicitlyConstructible_V<FTwoArgConstructor, FTwoArgConstructor>);
	static_assert(!TIsImplicitlyConstructible_V<FTwoArgConstructor, void*>);
	static_assert(!TIsImplicitlyConstructible_V<FTwoArgConstructor, const void*>);
	static_assert(TIsImplicitlyConstructible_V<FTwoArgConstructor, void*, bool>);
	static_assert(!TIsImplicitlyConstructible_V<FTwoArgConstructor, const void*, bool>);

	struct FTwoArgConstructorWithDefaults
	{
		FTwoArgConstructorWithDefaults(void* = nullptr, bool = true);
	};
	static_assert(TIsImplicitlyConstructible_V<FTwoArgConstructorWithDefaults>);
	static_assert(TIsImplicitlyConstructible_V<FTwoArgConstructorWithDefaults, FTwoArgConstructorWithDefaults>);
	static_assert(TIsImplicitlyConstructible_V<FTwoArgConstructorWithDefaults, void*>);
	static_assert(!TIsImplicitlyConstructible_V<FTwoArgConstructorWithDefaults, const void*>);
	static_assert(TIsImplicitlyConstructible_V<FTwoArgConstructorWithDefaults, void*, bool>);
	static_assert(!TIsImplicitlyConstructible_V<FTwoArgConstructorWithDefaults, const void*, bool>);

	struct FImplicitlyConvertible
	{
		operator FEmpty() const;
	};
	static_assert(!TIsImplicitlyConstructible_V<FImplicitlyConvertible, FEmpty>);
	static_assert(TIsImplicitlyConstructible_V<FEmpty, FImplicitlyConvertible>);

	struct FExplicitlyConvertible
	{
		explicit operator FEmpty() const;
	};
	static_assert(!TIsImplicitlyConstructible_V<FExplicitlyConvertible, FEmpty>);
	static_assert(!TIsImplicitlyConstructible_V<FEmpty, FExplicitlyConvertible>);
}

#endif // WITH_TESTS
