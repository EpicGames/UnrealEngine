// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "Traits/RemoveMemberPtr.h"
#include <type_traits>

#if WITH_TESTS

namespace
{
	struct FDummyType;

	// Types which aren't pointer-to-members should return the same type
	static_assert(std::is_same_v<TRemoveMemberPtr_T<void                   >, void                   >);
	static_assert(std::is_same_v<TRemoveMemberPtr_T<int32                  >, int32                  >);
	static_assert(std::is_same_v<TRemoveMemberPtr_T<const char*            >, const char*            >);
	static_assert(std::is_same_v<TRemoveMemberPtr_T<bool(const char*) const>, bool(const char*) const>);

	// Pointers to data members should return the data member type (with cv qualifiers)
	static_assert(std::is_same_v<TRemoveMemberPtr_T<               int32       FDummyType::*>,                int32      >);
	static_assert(std::is_same_v<TRemoveMemberPtr_T<const          int32       FDummyType::*>, const          int32      >);
	static_assert(std::is_same_v<TRemoveMemberPtr_T<      volatile int32       FDummyType::*>,       volatile int32      >);
	static_assert(std::is_same_v<TRemoveMemberPtr_T<const volatile int32       FDummyType::*>, const volatile int32      >);
	static_assert(std::is_same_v<TRemoveMemberPtr_T<const char*                FDummyType::*>, const char*               >);
	static_assert(std::is_same_v<TRemoveMemberPtr_T<const char* const          FDummyType::*>, const char* const         >);
	static_assert(std::is_same_v<TRemoveMemberPtr_T<const char*       volatile FDummyType::*>, const char*       volatile>);
	static_assert(std::is_same_v<TRemoveMemberPtr_T<const char* const volatile FDummyType::*>, const char* const volatile>);

	// Pointers to member function types should return the function type (with cv- and ref-qualifiers
	static_assert(std::is_same_v<TRemoveMemberPtr_T<void (FDummyType::*)()                                          >, void()                                          >);
	static_assert(std::is_same_v<TRemoveMemberPtr_T<void (FDummyType::*)()                          const           >, void()                          const           >);
	static_assert(std::is_same_v<TRemoveMemberPtr_T<void (FDummyType::*)()                                volatile  >, void()                                volatile  >);
	static_assert(std::is_same_v<TRemoveMemberPtr_T<void (FDummyType::*)()                          const volatile  >, void()                          const volatile  >);
	static_assert(std::is_same_v<TRemoveMemberPtr_T<void (FDummyType::*)()                                        & >, void()                                        & >);
	static_assert(std::is_same_v<TRemoveMemberPtr_T<void (FDummyType::*)()                          const         & >, void()                          const         & >);
	static_assert(std::is_same_v<TRemoveMemberPtr_T<void (FDummyType::*)()                                volatile& >, void()                                volatile& >);
	static_assert(std::is_same_v<TRemoveMemberPtr_T<void (FDummyType::*)()                          const volatile& >, void()                          const volatile& >);
	static_assert(std::is_same_v<TRemoveMemberPtr_T<void (FDummyType::*)()                                        &&>, void()                                        &&>);
	static_assert(std::is_same_v<TRemoveMemberPtr_T<void (FDummyType::*)()                          const         &&>, void()                          const         &&>);
	static_assert(std::is_same_v<TRemoveMemberPtr_T<void (FDummyType::*)()                                volatile&&>, void()                                volatile&&>);
	static_assert(std::is_same_v<TRemoveMemberPtr_T<void (FDummyType::*)()                          const volatile&&>, void()                          const volatile&&>);
	static_assert(std::is_same_v<TRemoveMemberPtr_T<bool (FDummyType::*)(float, int32, const char*)                 >, bool(float, int32, const char*)                 >);
	static_assert(std::is_same_v<TRemoveMemberPtr_T<bool (FDummyType::*)(float, int32, const char*) const           >, bool(float, int32, const char*) const           >);
	static_assert(std::is_same_v<TRemoveMemberPtr_T<bool (FDummyType::*)(float, int32, const char*)       volatile  >, bool(float, int32, const char*)       volatile  >);
	static_assert(std::is_same_v<TRemoveMemberPtr_T<bool (FDummyType::*)(float, int32, const char*) const volatile  >, bool(float, int32, const char*) const volatile  >);
	static_assert(std::is_same_v<TRemoveMemberPtr_T<bool (FDummyType::*)(float, int32, const char*)               & >, bool(float, int32, const char*)               & >);
	static_assert(std::is_same_v<TRemoveMemberPtr_T<bool (FDummyType::*)(float, int32, const char*) const         & >, bool(float, int32, const char*) const         & >);
	static_assert(std::is_same_v<TRemoveMemberPtr_T<bool (FDummyType::*)(float, int32, const char*)       volatile& >, bool(float, int32, const char*)       volatile& >);
	static_assert(std::is_same_v<TRemoveMemberPtr_T<bool (FDummyType::*)(float, int32, const char*) const volatile& >, bool(float, int32, const char*) const volatile& >);
	static_assert(std::is_same_v<TRemoveMemberPtr_T<bool (FDummyType::*)(float, int32, const char*)               &&>, bool(float, int32, const char*)               &&>);
	static_assert(std::is_same_v<TRemoveMemberPtr_T<bool (FDummyType::*)(float, int32, const char*) const         &&>, bool(float, int32, const char*) const         &&>);
	static_assert(std::is_same_v<TRemoveMemberPtr_T<bool (FDummyType::*)(float, int32, const char*)       volatile&&>, bool(float, int32, const char*)       volatile&&>);
	static_assert(std::is_same_v<TRemoveMemberPtr_T<bool (FDummyType::*)(float, int32, const char*) const volatile&&>, bool(float, int32, const char*) const volatile&&>);
}

#endif // WITH_TESTS
