// Copyright Epic Games, Inc. All Rights Reserved.
#include "UObject/FortniteMainBranchObjectVersion.h"

#include "FortniteMainBranchObjectVersion_Restricted.inl"

TMap<FGuid, FGuid> FFortniteMainBranchObjectVersion::GetSystemGuids()
{
	TMap<FGuid, FGuid> SystemGuids;
	const FDevSystemGuids& DevGuids = FDevSystemGuids::Get();

	SystemGuids.Add(DevGuids.GLOBALSHADERMAP_DERIVEDDATA_VER, FGuid("C340EEF623494448979F6D24B496E85A"));
	SystemGuids.Add(DevGuids.LANDSCAPE_MOBILE_COOK_VERSION, FGuid("32D02EF867C74B71A0D4E0FA41392732"));
	SystemGuids.Add(DevGuids.MATERIALSHADERMAP_DERIVEDDATA_VER, FGuid("49B25321CC344253B87962AD4236096E"));
	SystemGuids.Add(DevGuids.NIAGARASHADERMAP_DERIVEDDATA_VER, FGuid("87F6AE6C269A4592B281752A582E7F13"));
	SystemGuids.Add(DevGuids.Niagara_LatestScriptCompileVersion, FGuid("266FBE22ADCD4773B98124B68DBE3A4C"));
	SystemGuids.Add(DevGuids.MaterialTranslationDDCVersion, FGuid("B79683EE9E684C13BAC2FDE8B180477F"));

	// These GUIDs are in AppendRestrictedFortniteMainStreamObjectVersionGuids as they should rarely be changed.
	//SystemGuids.Add(DevGuids.NANITE_DERIVEDDATA_VER, FGuid("ACF98184978742A592537DA223A1E6BE"));
	//SystemGuids.Add(DevGuids.SkeletalMeshDerivedDataVersion, FGuid("C998F2F3B2884A02B7E290B8C25835F0"));
	//SystemGuids.Add(DevGuids.STATICMESH_DERIVEDDATA_VER, FGuid("3ABF32149F9D4D838750368AB95573BE"));
		
	AppendRestrictedFortniteMainStreamObjectVersionGuids(SystemGuids);
	return SystemGuids;
}
