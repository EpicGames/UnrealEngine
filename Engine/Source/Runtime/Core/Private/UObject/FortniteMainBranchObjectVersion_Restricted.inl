// Copyright Epic Games, Inc. All Rights Reserved.

static void AppendRestrictedFortniteMainStreamObjectVersionGuids(TMap<FGuid, FGuid>& SystemGuids)
{
	// Guids added here are expected to only be bumped after careful consideration of the
	// patch effects.
	const FDevSystemGuids& DevGuids = FDevSystemGuids::Get();
	SystemGuids.Add(DevGuids.NANITE_DERIVEDDATA_VER, FGuid("F365D7A48F454525AAADC0678EA085A0"));
	SystemGuids.Add(DevGuids.SkeletalMeshDerivedDataVersion, FGuid("A3F92C1D4E7B108AD61C8F4209B5E3F0"));
	SystemGuids.Add(DevGuids.STATICMESH_DERIVEDDATA_VER, FGuid("3ABF32149F9D4D838750368AB95573BE"));
}
