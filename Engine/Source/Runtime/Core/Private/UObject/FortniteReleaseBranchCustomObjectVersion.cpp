// Copyright Epic Games, Inc. All Rights Reserved.
#include "UObject/FortniteReleaseBranchCustomObjectVersion.h"

TMap<FGuid, FGuid> FFortniteReleaseBranchCustomObjectVersion::GetSystemGuids()
{
	TMap<FGuid, FGuid> SystemGuids;
	const FDevSystemGuids& DevGuids = FDevSystemGuids::Get();

	SystemGuids.Add(DevGuids.Niagara_LatestScriptCompileVersion, FGuid("213B319D239D438C82A946BB164152DE"));
	SystemGuids.Add(DevGuids.SkeletalMeshDerivedDataVersion, FGuid("84A80681A98747F3BD0C2257703712D6"));
	SystemGuids.Add(DevGuids.MaterialTranslationDDCVersion, FGuid("3CD389BC871B4FFF8351228666EAE54F"));
	SystemGuids.Add(DevGuids.GLOBALSHADERMAP_DERIVEDDATA_VER, FGuid("43D841E68BB14DF4A45F6D09B35501F7"));
	SystemGuids.Add(DevGuids.MATERIALSHADERMAP_DERIVEDDATA_VER, FGuid("D52C1D148C294A6A8C6CD75505081A31"));
	SystemGuids.Add(DevGuids.STATICMESH_DERIVEDDATA_VER, FGuid("C3F59BCDA9C84065BCF3A91BA0533B0E"));
	SystemGuids.Add(DevGuids.NANITE_DERIVEDDATA_VER, FGuid("2A6C3CB2933840498122C6B81BDCB33D"));
	return SystemGuids;
}