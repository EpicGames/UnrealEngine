// Copyright Epic Games, Inc. All Rights Reserved.
#include "UObject/FortniteSeasonBranchObjectVersion.h"

TMap<FGuid, FGuid> FFortniteSeasonBranchObjectVersion::GetSystemGuids()
{
	TMap<FGuid, FGuid> SystemGuids;
	const FDevSystemGuids& DevGuids = FDevSystemGuids::Get();

	SystemGuids.Add(DevGuids.STATICMESH_DERIVEDDATA_VER, FGuid("C8B712A947FA4A42B783183B1176DB99"));
	SystemGuids.Add(DevGuids.NANITE_DERIVEDDATA_VER, FGuid("08F9E554F3544CE7895C8A4D0FA8C7F0"));
	SystemGuids.Add(DevGuids.Niagara_LatestScriptCompileVersion, FGuid("08A58144744FCA4A9BF6DD45DFC4EC13"));

	return SystemGuids;
}
