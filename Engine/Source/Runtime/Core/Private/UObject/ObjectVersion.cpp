// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	UnObjVer.cpp: Unreal version definitions.
=============================================================================*/

#include "UObject/ObjectVersion.h"

#include "Misc/AssertionMacros.h"
#include "Serialization/Archive.h"
#include "Serialization/CompactBinary.h"
#include "Serialization/CompactBinarySerialization.h"
#include "Serialization/CompactBinaryWriter.h"

// @see ObjectVersion.h for the list of changes/defines

const FPackageFileVersion GPackageFileUEVersion(VER_LATEST_ENGINE_UE4, EUnrealEngineObjectUE5Version::AUTOMATIC_VERSION);
const FPackageFileVersion GOldestLoadablePackageFileUEVersion = FPackageFileVersion::CreateUE4Version(VER_UE4_OLDEST_LOADABLE_PACKAGE);

const int32 GPackageFileLicenseeUEVersion = VER_LATEST_ENGINE_LICENSEEUE4;

FPackageFileVersion FPackageFileVersion::CreateUE4Version(int32 Version)
{
	check(Version <= EUnrealEngineObjectUE4Version::VER_UE4_AUTOMATIC_VERSION);
	return FPackageFileVersion(Version, (EUnrealEngineObjectUE5Version)0);
}

FPackageFileVersion FPackageFileVersion::CreateUE4Version(EUnrealEngineObjectUE4Version Version)
{
	check(Version <= EUnrealEngineObjectUE4Version::VER_UE4_AUTOMATIC_VERSION);
	return FPackageFileVersion(Version, (EUnrealEngineObjectUE5Version)0);
}

FArchive& operator<<(FArchive& Ar, FPackageFileVersion& Version)
{
	Ar << Version.FileVersionUE4;
	Ar << Version.FileVersionUE5;

	return Ar;
}

FCbWriter& FPackageFileVersion::Write(FCbWriter& Writer) const
{
	Writer.BeginObject();
	Writer << "ue4version" << FileVersionUE4;
	Writer << "ue5version" << FileVersionUE5;
	Writer.EndObject();
	return Writer;
}

bool FPackageFileVersion::TryRead(const FCbFieldView& FieldView)
{
	bool bOk = LoadFromCompactBinary(FieldView["ue4version"], FileVersionUE4);
	bOk = LoadFromCompactBinary(FieldView["ue5version"], FileVersionUE5) & bOk;
	return bOk;
}
