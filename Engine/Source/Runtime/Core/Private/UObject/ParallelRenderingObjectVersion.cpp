// Copyright Epic Games, Inc. All Rights Reserved.
#include "UObject/ParallelRenderingObjectVersion.h"

TMap<FGuid, FGuid> FParallelRenderingObjectVersion::GetSystemGuids()
{
	TMap<FGuid, FGuid> SystemGuids;
	const FDevSystemGuids& DevGuids = FDevSystemGuids::Get();

	SystemGuids.Add(DevGuids.NANITE_DERIVEDDATA_VER, FGuid("6ED76D6068904EBB9B3ADAC259F0F53E"));
	SystemGuids.Add(DevGuids.STATICMESH_DERIVEDDATA_VER, FGuid("EB4D2761094040DD889CC1BE3D24E1B3"));
	SystemGuids.Add(DevGuids.MATERIALSHADERMAP_DERIVEDDATA_VER, FGuid("21EC97751B084536AE887AA52996DA1C"));
	return SystemGuids;
}
