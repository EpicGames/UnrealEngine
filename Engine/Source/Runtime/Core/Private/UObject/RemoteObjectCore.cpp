// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	RemoteObjectCore.cpp: Core module Remote Object utility functions.
=============================================================================*/

#include "UObject/RemoteObjectCore.h"

#include "Misc/AssertionMacros.h"

const TCHAR* EnumToString(EResidence InResidence)
{
	switch (InResidence)
	{
	case EResidence::Local:
		return TEXT("Local");
	case EResidence::LocalNotReady:
		return TEXT("LocalNotReady");
	case EResidence::Remote:
		return TEXT("Remote");
	default:
		checkf(false, TEXT("EnumToString: Unknown EResidence"));
		break;
	};
	return TEXT("");
}