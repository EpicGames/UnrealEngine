// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/ScriptDelegates.h"

#if UE_USE_DYNAMIC_DELEGATE_PAYLOADS

namespace UE::Core::Private
{
	FDelegatePayloadSerializeFunc       GDelegatePayloadSerializeFunc       = nullptr;
	FDelegatePayloadDeserializeFunc     GDelegatePayloadDeserializeFunc     = nullptr;
	FDelegatePayloadSerializeSlotFunc   GDelegatePayloadSerializeSlotFunc   = nullptr;
	FDelegatePayloadDeserializeSlotFunc GDelegatePayloadDeserializeSlotFunc = nullptr;
}

#endif
