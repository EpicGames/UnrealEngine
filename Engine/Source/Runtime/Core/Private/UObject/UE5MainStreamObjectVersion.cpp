// Copyright Epic Games, Inc. All Rights Reserved.
#include "UObject/UE5MainStreamObjectVersion.h"
#include "Containers/Map.h"
#include "Misc/Guid.h"

#include "UE5MainStreamObjectVersion_Restricted.inl"

TMap<FGuid, FGuid> FUE5MainStreamObjectVersion::GetSystemGuids()
{
	TMap<FGuid, FGuid> SystemGuids;
	const FDevSystemGuids& DevGuids = FDevSystemGuids::Get();

	SystemGuids.Add(DevGuids.GLOBALSHADERMAP_DERIVEDDATA_VER, FGuid("FEAB692FBBDD4113857BCD32C2B644BC"));
	SystemGuids.Add(DevGuids.LANDSCAPE_MOBILE_COOK_VERSION, FGuid("71000000000000000000000000000035"));
	SystemGuids.Add(DevGuids.MATERIALSHADERMAP_DERIVEDDATA_VER, FGuid("D28F2FDCCAC4499B8F8E87728235A46C"));
	SystemGuids.Add(DevGuids.NIAGARASHADERMAP_DERIVEDDATA_VER, FGuid("8A37C45D24F2423CBE5F8F371DE33575"));
	SystemGuids.Add(DevGuids.Niagara_LatestScriptCompileVersion, FGuid("9C8812406D824DE29BBC6F303964BB7E"));
	SystemGuids.Add(DevGuids.POSESEARCHDB_DERIVEDDATA_VER, FGuid("4E595C2AC5E947D6BA9ABC874353E5BC"));
	SystemGuids.Add(DevGuids.GROOM_BINDING_DERIVED_DATA_VERSION, FGuid("156678D4F9084D7CAE98FADC6AB93573"));
	SystemGuids.Add(DevGuids.GROOM_DERIVED_DATA_VERSION, FGuid("00B8207248CC44B88F7DEC8F328A5D8B"));
	SystemGuids.Add(DevGuids.MaterialTranslationDDCVersion, FGuid("5E6E21D1AC364AACA1921EE46BF10B4C"));

	// These GUIDS are in restricted as they should rarely be changed.
	//SystemGuids.Add(DevGuids.NANITE_DERIVEDDATA_VER, FGuid("49D2EF76967340EFB22F2F398E4DDDDD"));
	//SystemGuids.Add(DevGuids.SkeletalMeshDerivedDataVersion, FGuid("CFE0AF3B4F48465E8DE6A6BD81881F18"));
	//SystemGuids.Add(DevGuids.STATICMESH_DERIVEDDATA_VER, FGuid("98F7D79A5811013825E75BBCC41ED3E9"));
		
	AppendRestrictedUE5MainStreamObjectVersionGuids(SystemGuids);

	return SystemGuids;
}
