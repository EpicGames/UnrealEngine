// Copyright Epic Games, Inc. All Rights Reserved.

static void AppendRestrictedUE5MainStreamObjectVersionGuids(TMap<FGuid, FGuid>& SystemGuids)
{
	// Guids added here are expected to only be bumped after careful consideration of the
	// patch effects.
	const FDevSystemGuids& DevGuids = FDevSystemGuids::Get();
	SystemGuids.Add(DevGuids.NANITE_DERIVEDDATA_VER, FGuid("10723B2C576B477590594B071647B4B7"));
	SystemGuids.Add(DevGuids.SkeletalMeshDerivedDataVersion, FGuid("D107D9EFBABA44B3B404E1D77243F3DD"));
	SystemGuids.Add(DevGuids.STATICMESH_DERIVEDDATA_VER, FGuid("98F7D79A5811013825E75BBCC41ED3E9"));
}
