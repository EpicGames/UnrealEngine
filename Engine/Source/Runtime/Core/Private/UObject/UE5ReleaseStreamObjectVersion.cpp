// Copyright Epic Games, Inc. All Rights Reserved.
#include "UObject/UE5ReleaseStreamObjectVersion.h"
#include "Containers/Map.h"
#include "Misc/Guid.h"

TMap<FGuid, FGuid> FUE5ReleaseStreamObjectVersion::GetSystemGuids()
{
	TMap<FGuid, FGuid> SystemGuids;
	const FDevSystemGuids& DevGuids = FDevSystemGuids::Get();

	SystemGuids.Add(DevGuids.GLOBALSHADERMAP_DERIVEDDATA_VER, FGuid("AACA8C54724D4E9386927BA2A165ACA6"));
	SystemGuids.Add(DevGuids.MATERIALSHADERMAP_DERIVEDDATA_VER, FGuid("6727B49695474CCC8BC1C1B211A6B83A"));
	SystemGuids.Add(DevGuids.NANITE_DERIVEDDATA_VER, FGuid("AC88CFBDDA614A9C8488F64D2DAF699D"));
	SystemGuids.Add(DevGuids.NIAGARASHADERMAP_DERIVEDDATA_VER, FGuid("26325805F5744BC2815552A21D95CECA"));
	SystemGuids.Add(DevGuids.Niagara_LatestScriptCompileVersion, FGuid("14BEE7BD97194A7BBB8374D34DC49915"));
	SystemGuids.Add(DevGuids.SkeletalMeshDerivedDataVersion, FGuid("BA91E71F642E4813A1B74B80D0448928"));
	SystemGuids.Add(DevGuids.STATICMESH_DERIVEDDATA_VER, FGuid("14B86848953D46989C69A8B195AD7F6C"));

	return SystemGuids;
}
