// Copyright Epic Games, Inc. All Rights Reserved.
#include "UObject/UE5SpecialProjectStreamObjectVersion.h"
#include "UObject/DevObjectVersion.h"
#include "Containers/Map.h"
#include "Misc/Guid.h"


TMap<FGuid, FGuid> FUE5SpecialProjectStreamObjectVersion::GetSystemGuids()
{
	TMap<FGuid, FGuid> SystemGuids;
	const FDevSystemGuids& DevGuids = FDevSystemGuids::Get();

	SystemGuids.Add(DevGuids.NANITE_DERIVEDDATA_VER, FGuid("C73C42D8-4CA0-4EAF-8968-722EB9EEF812"));
	SystemGuids.Add(DevGuids.SkeletalMeshDerivedDataVersion, FGuid("56B88323-F44D-46B8-A3A3-B3441BC8F448"));
	SystemGuids.Add(DevGuids.MaterialTranslationDDCVersion, FGuid("BFB09A107AD8451294FFD3B321C3565C"));

	return SystemGuids;
}
