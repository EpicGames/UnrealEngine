// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/UnrealNames.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "UObject/NameBatchSerialization.h"
#include "Async/Async.h"
#include "Misc/AssertionMacros.h"
#include "Misc/MessageDialog.h"
#include "Misc/OutputDeviceHelper.h"
#include "Misc/Paths.h"
#include "Math/NumericLimits.h"
#include "Math/UnrealMathUtility.h"
#include "HAL/UnrealMemory.h"
#include "Templates/UnrealTemplate.h"
#include "Misc/CString.h"
#include "Misc/Crc.h"
#include "Misc/StringBuilder.h"
#include "Misc/TransactionallySafeRWLock.h"
#include "Containers/UnrealString.h"
#include "Containers/Utf8String.h"
#include "Containers/SharedString.h"
#include "UObject/NameTypes.h"
#include "Logging/LogMacros.h"
#include "Misc/ByteSwap.h"
#include "UObject/ObjectVersion.h"
#include "HAL/ThreadSafeCounter.h"
#include "Misc/ScopeRWLock.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "ProfilingDebugging/StringsTrace.h"
#include "ProfilingDebugging/MetadataTrace.h"
#include "Containers/Set.h"
#include "Internationalization/Text.h"
#include "Internationalization/TextChar.h"
#include "Internationalization/Internationalization.h"
#include "Misc/OutputDeviceRedirector.h"
#include "HAL/FileManager.h"
#include "HAL/IConsoleManager.h"
#include "HAL/LowLevelMemTracker.h"
#include "Serialization/MemoryImage.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "Hash/Blake3.h"
#include "Hash/CityHash.h"
#include "Templates/AlignmentTemplates.h"
#include "Templates/Greater.h"
#include "Misc/AsciiSet.h"
#include "AutoRTFM.h"

#if WITH_TESTS
#include "Tests/TestHarnessAdapter.h"
#endif

#ifndef USE_FNAME_MMAP
#define USE_FNAME_MMAP 0
#endif

#if USE_FNAME_MMAP
#include <sys/mman.h>
#if PLATFORM_ANDROID
extern FString AndroidThunkCpp_GetCacheDir();
#endif
#endif

PRAGMA_DISABLE_UNSAFE_TYPECAST_WARNINGS

DEFINE_LOG_CATEGORY_STATIC(LogUnrealNames, Log, All);

static_assert(::HasIntrusiveUnsetOptionalState<FName>());

// Console command declarations
namespace UE::Name::Private
{
	// List FNames to the output device 
	void ListFNames(const TArray<FString>&, UWorld*, FOutputDevice&);
	// Dump FNames to a file
	void DumpFNames(const TArray<FString>&, UWorld*, FOutputDevice&);

	// List FNames to the output device 
	void ListNumberedFNames(const TArray<FString>&, UWorld*, FOutputDevice&);
	// Dump FNames to a file
	void DumpNumberedFNames(const TArray<FString>&, UWorld*, FOutputDevice&);

	// Write FName stats to the output device 
	void GetFNameStats(FOutputDevice&);

	// Write csv stats on hash distribution
	void DumpHashCsv(FOutputDevice&);

	FAutoConsoleCommandWithWorldArgsAndOutputDevice CCListFNames(
		TEXT("FName.List"),
		TEXT("List all base FName strings to the output device. Pass -num=n to list the most recent n names."),
		FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateStatic(&ListFNames)
	);
	FAutoConsoleCommandWithWorldArgsAndOutputDevice CCDumpFNames(
		TEXT("FName.Dump"),
		TEXT("Dump all base FName strings to a file. Pass -num=n to dump the most recent n names."),
		FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateStatic(&DumpFNames)
	);
	FAutoConsoleCommandWithWorldArgsAndOutputDevice CCListNumberedFNames(
		TEXT("FName.ListNumbered"),
		TEXT("List all numbered FNames to the output devicce (only when UE_FNAME_OUTLINE_NUMBER is set). Pass -num=n to list the most recent n names."),
		FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateStatic(&ListNumberedFNames)
	);
	FAutoConsoleCommandWithWorldArgsAndOutputDevice CCDumpNumberedFNames(
		TEXT("FName.DumpNumbered"),
		TEXT("Dump all numbered FNames to a file (only when UE_FNAME_OUTLINE_NUMBER is set). Pass -num=n to dump the most recent n names."),
		FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateStatic(&DumpNumberedFNames)
	);
	FAutoConsoleCommandWithOutputDevice CCGetFNameStats(
		TEXT("FName.Stats"),
		TEXT("Write FName stats to the output device."),
		FConsoleCommandWithOutputDeviceDelegate::CreateStatic(&GetFNameStats)
	);
	FAutoConsoleCommandWithOutputDevice CCDumpHashCsv(
		TEXT("FName.HashCsv"),
		TEXT("Write FName hash stats to a csv file."),
		FConsoleCommandWithOutputDeviceDelegate::CreateStatic(&DumpHashCsv)
	);
}

bool GFNameUseMmap = false;
static FAutoConsoleVariableRef CVarFNameUseMmap(
	TEXT("FName.UseMmap"),
	GFNameUseMmap,
	TEXT("Whether to use mmap instead of malloc for block allocations"));

const TCHAR* LexToString(EName Ename)
{
	switch (Ename)
	{
#define REGISTER_NAME(num,name) case EName::name: return TEXT(UE_STRINGIZE(name));
#include "UObject/UnrealNames.inl"
#undef REGISTER_NAME
		default:
			return TEXT("*INVALID*");
	}
}

struct FNameEntryIds
{
	FNameEntryId ComparisonId, DisplayId;
};

/*-----------------------------------------------------------------------------
	FName helpers. 
-----------------------------------------------------------------------------*/

static bool operator==(FNameEntryHeader A, FNameEntryHeader B)
{
	static_assert(sizeof(FNameEntryHeader) == 2, "");
	return (uint16&)A == (uint16&)B;
}

template<typename FromCharType, typename ToCharType>
ToCharType* ConvertInPlace(FromCharType* Str, uint32 Len)
{
	static_assert(std::is_same_v<FromCharType, ToCharType>, "Unsupported conversion");
	return Str;
}

template<>
WIDECHAR* ConvertInPlace<ANSICHAR, WIDECHAR>(ANSICHAR* Str, uint32 Len)
{
	for (uint32 Index = Len; Index--; )
	{
		reinterpret_cast<WIDECHAR*>(Str)[Index] = Str[Index];
	}

	return reinterpret_cast<WIDECHAR*>(Str);
}

template<>
ANSICHAR* ConvertInPlace<WIDECHAR, ANSICHAR>(WIDECHAR* Str, uint32 Len)
{
	for (uint32 Index = 0; Index < Len; ++Index)
	{
		reinterpret_cast<ANSICHAR*>(Str)[Index] = Str[Index];
	}

	return reinterpret_cast<ANSICHAR*>(Str);
}

union FNameBuffer
{
	ANSICHAR AnsiName[NAME_SIZE];
	WIDECHAR WideName[NAME_SIZE];
};

struct FNameStringView
{
	FNameStringView() : Data(nullptr), Len(0), bIsWide(false) {}
	FNameStringView(const ANSICHAR* Str, uint32 InLen) : Ansi(Str), Len(InLen), bIsWide(false) {}
	FNameStringView(const WIDECHAR* Str, uint32 InLen) : Wide(Str), Len(InLen), bIsWide(true) {}
	FNameStringView(const void* InData, uint32 InLen, bool bInIsWide) : Data(InData), Len(InLen), bIsWide(bInIsWide) {}

	union
	{
		const void* Data;
		const ANSICHAR* Ansi;
		const WIDECHAR* Wide;
	};

	uint32 Len;
	bool bIsWide;

	bool IsAnsi() const { return !bIsWide; }
	bool IsNoneString() const;

	int32 BytesWithTerminator() const
	{
		return (Len + 1) * (bIsWide ? sizeof(WIDECHAR) : sizeof(ANSICHAR));
	}

	int32 BytesWithoutTerminator() const
	{
		return Len * (bIsWide ? sizeof(WIDECHAR) : sizeof(ANSICHAR));
	}
};

template<ENameCase Sensitivity>
FORCEINLINE bool EqualsSameDimensions(FNameStringView A, FNameStringView B)
{
	checkSlow(A.Len == B.Len && A.IsAnsi() == B.IsAnsi());

	int32 Len = A.Len;

	if (Sensitivity == ENameCase::CaseSensitive)
	{
		return B.IsAnsi() ? !FPlatformString::Strncmp(A.Ansi, B.Ansi, Len) : !FPlatformString::Strncmp(A.Wide, B.Wide, Len);
	}
	else
	{
		return B.IsAnsi() ? !FPlatformString::Strnicmp(A.Ansi, B.Ansi, Len) : !FPlatformString::Strnicmp(A.Wide, B.Wide, Len);
	}

}

template<ENameCase Sensitivity>
FORCEINLINE bool Equals(FNameStringView A, FNameStringView B)
{
	return (A.Len == B.Len & A.IsAnsi() == B.IsAnsi()) && EqualsSameDimensions<Sensitivity>(A, B);
}

template<ENameCase Sensitivity>
bool EqualsSameDimensions(const FNameEntry& Entry, FNameStringView Name)
{
	FNameBuffer DecodeBuffer;
	return EqualsSameDimensions<Sensitivity>(Entry.MakeView(DecodeBuffer), Name);
}

/** Remember to update natvis if you change these */
static constexpr uint32 FNameMaxBlockBits = 13; // Limit block array a bit, still allowing 8k * block size = 1GB - 2G of FName entry data
static constexpr uint32 FNameBlockOffsetBits = 16;
static constexpr uint32 FNameMaxBlocks = 1 << FNameMaxBlockBits;
static constexpr uint32 FNameBlockOffsets = 1 << FNameBlockOffsetBits;
static constexpr uint32 FNameEntryIdBits = FNameBlockOffsetBits + FNameMaxBlockBits;
static constexpr uint32 FNameEntryIdMask = (1 << FNameEntryIdBits ) - 1;

// Stores unterminated strings in FNameEntry::AnsiName/WideName
struct FDefaultNameStore
{
	// Decides if entry header + Name data fits in OutSize. Called inside allocator and shard lock.
	// Implementatations should not consider alignment of FNameEntry, that's the responsibility of the caller
	//
	// @return Entry size desired by this name store if Name fits, 0 otherwise. Allocator will align up if necessary.
	static uint32 TryPlace(uint8* OutEntry, uint32 OutSize, FNameStringView Name)
	{
		uint32 Bytes = GetDefaultSize(Name.Len, Name.bIsWide);
		return Bytes <= OutSize ? Bytes : 0;
	}

	// Called inside shard lock but outside allocator lock to avoid contention
	static void Finalize(uint8* OutNameData, FNameStringView Name)
	{
		FMemory::Memcpy(OutNameData, Name.Data, Name.BytesWithoutTerminator());
	}
	
	static void Copy(ANSICHAR* Out, const uint8* NameData, uint32 NameLen)
	{
		FMemory::Memcpy(Out, NameData, sizeof(ANSICHAR) * NameLen);
	}

	static void Copy(WIDECHAR* Out, const uint8* NameData, uint32 NameLen)
	{
		FMemory::Memcpy(Out, NameData, sizeof(WIDECHAR) * NameLen);
	}

	static uint32 GetSize(const FNameEntry& Entry)
	{
		return GetDefaultSize(uint32(Entry.GetNameLength()), Entry.IsWide());
	}

	static const ANSICHAR* TryGetAnsi(const void* NameData)
	{
		return static_cast<const ANSICHAR*>(NameData);
	}
	
	static const WIDECHAR* TryGetWide(const void* NameData)
	{
		return static_cast<const WIDECHAR*>(NameData);
	}
	
private:
	static uint32 GetDefaultSize(uint32 Len, bool bIsWide)
	{
		uint32 HeaderSize = FNameEntry::GetDataOffset();
		uint32 FooterSize =  Len * (bIsWide ? sizeof(WIDECHAR) : sizeof(ANSICHAR));
		return HeaderSize + FooterSize;
	}
};

// To customize how FNames are encoded in memory:
// - Define UE_FNAME_CUSTOM_STORE_HEADER to point to a .inl source file
// - Implement a class that can override FDefaultNameStore in that file
// - End the included file with using FNameStore = FMyCustomNameStore
#ifdef UE_FNAME_CUSTOM_STORE_HEADER
#include UE_FNAME_CUSTOM_STORE_HEADER
#else
using FNameStore = FDefaultNameStore;
#endif
#ifdef WITH_CUSTOM_NAME_ENCODING
#pragma error "WITH_CUSTOM_NAME_ENCODING macro is deprecated. Use UE_FNAME_CUSTOM_STORE_HEADER to change how FName data is saved to memory."
#endif

/** An unpacked FNameEntryId */
struct FNameEntryHandle
{
	uint32 Block = 0;
	uint32 Offset = 0;

	FNameEntryHandle(uint32 InBlock, uint32 InOffset)
		: Block(InBlock)
		, Offset(InOffset)
	{
		checkName(Block < FNameMaxBlocks);
		checkName(Offset < FNameBlockOffsets);
	}

	FNameEntryHandle(FNameEntryId Id)
		: Block(Id.ToUnstableInt() >> FNameBlockOffsetBits)
		, Offset(Id.ToUnstableInt() & (FNameBlockOffsets - 1))
	{
	}

	operator FNameEntryId() const
	{
		return FNameEntryId::FromUnstableInt(Block << FNameBlockOffsetBits | Offset);
	}

	explicit operator bool() const { return Block | Offset; }
};

FArchive& operator<<(FArchive& Ar, FNameEntryId& Id)
{
	if (Ar.IsLoading())
	{
		uint32 UnstableInt = 0;
		Ar << UnstableInt;
		Id = FNameEntryId::FromUnstableInt(UnstableInt);
	}
	else
	{
		uint32 UnstableInt = Id.ToUnstableInt();
		Ar << UnstableInt;
	}

	return Ar;
}


template<uint32 NumBits>
class TNameAtomicBitSet
{
	using WordType = uint64;
	static constexpr uint32 BitsPerWord = 8 * sizeof(WordType);
	static constexpr uint32 NumWords = NumBits / BitsPerWord;
	static constexpr uint32 BitIndexMask = BitsPerWord - 1;
	static_assert((NumBits % BitsPerWord) == 0);
	
public:
	/**
	 * Atomically set a bit in the bit set.
	 * @param Index Index of bit to set
	 * @return Previous value of bit
	 */
	bool SetBitAtomic(uint32 Index)
	{
		const uint32 WordIndex = Index / BitsPerWord;
		const uint32 BitIndex = Index & BitIndexMask;
		const WordType ValueMask = WordType(1) << BitIndex;
		volatile WordType* DestDword = GetBuffer() + WordIndex;
		checkSlow(DestDword >= GetBuffer() && DestDword < (GetBuffer() + NumWords));
		const WordType PrevDwordValue = std::atomic_fetch_or((std::atomic<WordType>*)DestDword, ValueMask);
		return !!(PrevDwordValue & ValueMask);
	}
	
	/**
	 * Atomically clear a bit in the bit set.
	 * @param Index Index of bit to clear
	 * @return Previous value of bit
	 */
	bool ClearBitAtomic(uint32 Index)
	{
		const uint32 WordIndex = Index / BitsPerWord;
		const uint32 BitIndex = Index & BitIndexMask;
		const WordType ValueMask = WordType(1) << BitIndex;
		volatile WordType* DestDword = GetBuffer() + WordIndex;
		checkSlow(DestDword >= GetBuffer() && DestDword < (GetBuffer() + NumWords));
		const WordType PrevDwordValue = std::atomic_fetch_and((std::atomic<WordType>*)DestDword, ~ValueMask);
		return !!(PrevDwordValue & ValueMask);
	}

	/**
	 * Iterates over the set and issues the lambda for each index that is set.
	 * @param Func Functor to call
	 */
	template<typename Lambda>
	void ForEachSetBit(Lambda Func) const
	{
		const WordType* Words = Data.load(std::memory_order_relaxed);
		if (!Words)
		{
			return;
		}

		uint32 WordIndex(0);
		for (std::atomic<WordType> AtomicWord : MakeArrayView(Words, NumWords))
		{
			const WordType Word = AtomicWord.load();
			for (uint32 BitIdx = 0; BitIdx < BitsPerWord; ++BitIdx)
			{
				if (Word & (WordType(1) << BitIdx))
				{
					Func(BitIdx + (WordIndex * BitsPerWord));
				}
			}
			++WordIndex;
		}
	}
	
private:


	/**
	 * Returns or create the buffer.
	 * @return Pointer to buffer
	 */
	WordType* GetBuffer()
	{
		WordType* Buffer = Data.load(std::memory_order_relaxed);
		if (!Buffer)
		{
			LLM_SCOPE(ELLMTag::FName);
			LLM_TAGSET_SCOPE_CLEAR(ELLMTagSet::Assets);
			LLM_TAGSET_SCOPE_CLEAR(ELLMTagSet::AssetClasses);
			UE_TRACE_METADATA_CLEAR_SCOPE();
			constexpr uint32 BufferSizeBytes = NumWords * sizeof(WordType);
			Buffer = (WordType*) FMemory::MallocZeroed(BufferSizeBytes, alignof(WordType));
			WordType* Expected = nullptr;
			if (UNLIKELY(!Data.compare_exchange_strong(Expected, Buffer)))
			{
				// Another thread assigned the value before us. Free our buffer and use
				// the buffer from the other thread.
				FMemory::Free(Buffer);
				Buffer = Expected;
			}
		}
		return Buffer;
	}
	
	std::atomic<WordType*> Data{};
};

struct FNameSlot
{
	// Use the remaining few bits to store a hash that can determine inequality
	// during probing without touching entry data
	static constexpr uint32 EntryIdBits = FNameMaxBlockBits + FNameBlockOffsetBits;
	static constexpr uint32 EntryIdMask = (1 << EntryIdBits) - 1;
	static constexpr uint32 ProbeHashShift = EntryIdBits;
	static constexpr uint32 ProbeHashMask = ~EntryIdMask;
	
	FNameSlot() {}
	FNameSlot(FNameEntryId Value, uint32 ProbeHash)
		: IdAndHash(Value.ToUnstableInt() | ProbeHash)
	{
		check(!(Value.ToUnstableInt() & ProbeHashMask) && !(ProbeHash & EntryIdMask) && Used());
	}

	FNameEntryId GetId() const { return FNameEntryId::FromUnstableInt(IdAndHash & EntryIdMask); }
	uint32 GetProbeHash() const { return IdAndHash & ProbeHashMask; }
	
	bool operator==(FNameSlot Rhs) const { return IdAndHash == Rhs.IdAndHash; }

	bool Used() const { return !!IdAndHash;  }
private:
	uint32 IdAndHash = 0;
};

// Stores a (string, number) fname pair to construct the string "string_(number-1)" on demand.
// Id is a reference to another entry with Header.Len != 0 (no recursion).
struct FNumberedEntry
{
#if UE_FNAME_OUTLINE_NUMBER // Let empty type exist for clang static analyzer
	FNameEntryId	Id;
	uint32			Number;

	bool IsEmpty() const { return Id.IsNone() && Number == 0; }
#endif
};

/**
 * Thread-safe paged FNameEntry allocator
 */
class FNameEntryAllocator
{
public:
	enum { Stride = alignof(FNameEntry) };
	enum { BlockSizeBytes = Stride * FNameBlockOffsets };

#if USE_FNAME_MMAP
	const uint32 PageAlignedBlockSizeBytes = Align((uint32)BlockSizeBytes, (uint32)sysconf(_SC_PAGESIZE));
	// mmap block aligned to FName block size and page size
	const uint32 MmapSizeBytes = Align(128 * 1024 * 1024, PageAlignedBlockSizeBytes);

	bool IsBlockMmapped(uint8* BlockAddress)
	{
		for (uint8* Address : MmappedAddresses)
		{
			if (BlockAddress >= Address && BlockAddress < (Address + MmapSizeBytes))
			{
				return true;
			}
		}
		return false;
	}

#endif

	/** Initializes all member variables. */
	FNameEntryAllocator()
	{
		Blocks[0] = AllocBlock();
	}

	~FNameEntryAllocator()
	{
		for (int32 Index = CurrentBlock; Index >= 0; --Index)
		{
#if USE_FNAME_MMAP
			if (IsBlockMmapped(Blocks[Index]))
			{
				continue;
			}
#endif
			FMemory::Free(Blocks[Index]);
		}
#if USE_FNAME_MMAP
		for (uint8* Address : MmappedAddresses)
		{
			munmap(Address, MmapSizeBytes);
		}
		if (MmapFile != -1)
		{
			close(MmapFile);
		}
#endif
	}

	void ReserveBlocks(uint32 Num)
	{
		FWriteScopeLock _(Lock);

		for (uint32 Idx = Num - 1; Idx > CurrentBlock && Blocks[Idx] == nullptr; --Idx)
		{
			Blocks[Idx] = AllocBlock();
		}
	}
 
	// @param Bytes fits within current block
	inline FNameEntryHandle Allocate(uint32 Bytes)
	{
		checkSlow(CurrentByteCursor % Stride == 0);
		checkSlow(CurrentByteCursor + Bytes <= BlockSizeBytes);
 
		// Use current cursor position for this allocation and increment cursor for next allocation
		uint32 ByteOffset = CurrentByteCursor;
		// Align up so byte offset can be encoded in FNameEntryId and store waste 
		uint32 Step = Align(Bytes, alignof(FNameEntry));
		Waste += Step - Bytes;
		CurrentByteCursor += Step;
		return FNameEntryHandle(CurrentBlock, ByteOffset / Stride);
	}
 
#if UE_FNAME_OUTLINE_NUMBER
	template <class ScopeLock>
	FNameEntryHandle AllocateNumbered()
	{
		ScopeLock _(Lock);
 
		PadNumberedEntry();
		if (CurrentByteCursor + NumberedEntrySize > BlockSizeBytes)
		{
			AllocateNewBlock();
			PadNumberedEntry();
		}
 
		return Allocate(NumberedEntrySize);
	}
#endif
 
	// Allocate and store Name
	template <class ScopeLock>
	FNameEntryHandle AllocateRegular(FNameStringView Name)
	{
		ScopeLock _(Lock);
 
		uint32 Bytes = BlockSizeBytes < CurrentByteCursor ? 0 : FNameStore::TryPlace(Blocks[CurrentBlock] + CurrentByteCursor, BlockSizeBytes - CurrentByteCursor, Name);
 
		// Allocate a new block if current one is exhausted. We don't worry about a little bit
		// of waste at the end given the relative size of block to average and max allocation.
		if (Bytes == 0)
		{
			AllocateNewBlock();
			Bytes = FNameStore::TryPlace(Blocks[CurrentBlock] + CurrentByteCursor, BlockSizeBytes - CurrentByteCursor, Name);
			check(Bytes);
		}
 
		return Allocate(Bytes); 
	}

	template<class ScopeLock>
	FNameEntryHandle Create(FNameStringView Name, TOptional<FNameEntryId> ComparisonId, FNameEntryHeader Header)
	{
		LLM_SCOPE(ELLMTag::FName);
		FPlatformMisc::Prefetch(Blocks[CurrentBlock]);

		FNameEntryHandle Handle = AllocateRegular<ScopeLock>(Name);
		FNameEntry& Entry = Resolve(Handle);

#if WITH_CASE_PRESERVING_NAME
		Entry.ComparisonId = ComparisonId.IsSet() ? ComparisonId.GetValue() : FNameEntryId(Handle);
#endif

		Entry.Header = Header;
		FNameStore::Finalize(Entry.NameData, Name);

		return Handle;
	}

#if UE_FNAME_OUTLINE_NUMBER
	static_assert(sizeof(FNameEntry::FNumberedData::Id)	== sizeof(FNumberedEntry::Id) && sizeof(FNameEntry::FNumberedData::Number) == sizeof(FNumberedEntry::Number));
	static_assert(offsetof(FNameEntry::FNumberedData, Number) - offsetof(FNameEntry::FNumberedData,	Id) == offsetof(FNumberedEntry,	Number) - offsetof(FNumberedEntry, Id));
	static constexpr uint32 NumberedEntrySize = offsetof(FNameEntry, NumberedName) + sizeof(FNameEntry::NumberedName);

	template<class ScopeLock>
	FNameEntryHandle CreateWithNumber(const FNameEntryId& StringPart, uint32 NumberPart, TOptional<FNameEntryId> ComparisonId, FNameEntryHeader Header)
	{
		LLM_SCOPE(ELLMTag::FName);
		FPlatformMisc::Prefetch(Blocks[CurrentBlock]);
		FNameEntryHandle Handle = AllocateNumbered<ScopeLock>();
		FNameEntry& Entry = Resolve(Handle);

#if WITH_CASE_PRESERVING_NAME
		Entry.ComparisonId = ComparisonId.IsSet() ? ComparisonId.GetValue() : FNameEntryId(Handle);
#endif

		Entry.Header = Header;
		checkf((uint64)&Entry.GetNumberedName() % alignof(FNumberedEntry) == 0, TEXT("Failed to align numbered FName data"));
		reinterpret_cast<FNumberedEntry&>(Entry.NumberedName.Id[0]) = {StringPart, NumberPart};

		return Handle;
	}
#endif // UE_FNAME_OUTLINE_NUMBER

	FNameEntry& Resolve(FNameEntryHandle Handle) const
	{
		// Lock not needed
		return *reinterpret_cast<FNameEntry*>(Blocks[Handle.Block] + Stride * Handle.Offset);
	}

	void BatchLock() const
	{
		Lock.WriteLock();
	}

	void BatchUnlock() const
	{
		Lock.WriteUnlock();
	}

	/** Returns the number of blocks that have been allocated so far for names. */
	uint32 NumBlocks() const
	{
		return CurrentBlock + 1;
	}

	int64 GetWaste() const 
	{
		return Waste;
	}

	uint8** GetBlocksForDebugVisualizer() { return (uint8**)Blocks; }

	void DebugDump(TArray<const FNameEntry*>& Out) const
	{
		FRWScopeLock _(Lock, FRWScopeLockType::SLT_ReadOnly);

		for (uint32 BlockIdx = 0; BlockIdx < CurrentBlock; ++BlockIdx)
		{
			DebugDumpBlock(Blocks[BlockIdx], BlockSizeBytes, Out);
		}

		DebugDumpBlock(Blocks[CurrentBlock], CurrentByteCursor, Out);
	}

private:

#if UE_FNAME_OUTLINE_NUMBER
	static constexpr bool IsUnalignedNumberedEntry(uint32 BlockOffset)
	{
		constexpr bool bRuntimeAlign = alignof(FNameEntry) < alignof(FNumberedEntry);
		return bRuntimeAlign && (BlockOffset + offsetof(FNameEntry, NumberedName) + offsetof(FNameEntry::FNumberedData, Id)) % alignof(FNumberedEntry) != 0;
	}
	
	static constexpr FNameEntryHeader NumberPadHeader = {};

	// Pad up FNameEntry::NumberedName to be 4B-aligned if FNameEntry is only 2B-aligned
	void PadNumberedEntry()
	{
		if (IsUnalignedNumberedEntry(CurrentByteCursor) && (BlockSizeBytes - CurrentByteCursor) >= sizeof(FNameEntryHeader))
		{
			(FNameEntryHeader&)Blocks[CurrentBlock][CurrentByteCursor] = NumberPadHeader;
			CurrentByteCursor += sizeof(NumberPadHeader);
		}	
	}
#endif

	static void DebugDumpBlock(const uint8* Block, uint32 BlockSize, TArray<const FNameEntry*>& Out)
	{
		const uint8* It = Block;
		const uint8* End = It + BlockSize - FNameEntry::GetDataOffset();
		while (It < End)
		{
			const FNameEntry* Entry = (const FNameEntry*)It;
			if (uint32 Len = Entry->Header.Len)
			{
				Out.Add(Entry);
				It += Entry->GetSizeInBytes();
			}
#if !UE_FNAME_OUTLINE_NUMBER
			else
			{
				break; // Normal terminator with Header.Len == 0
			}
#else
			else if (End - It < NumberedEntrySize || Entry->GetNumberedName().IsEmpty())
			{
				break; // Numbered terminator or no room left for one
			}
			else if (IsUnalignedNumberedEntry(It - Block))
			{
				check(!WITH_CASE_PRESERVING_NAME);
				check(Entry->Header == NumberPadHeader);
				It += sizeof(NumberPadHeader);
			}
			else
			{
				Out.Add(Entry);
				It += NumberedEntrySize;
			}
#endif
		}
	}

	uint8* AllocBlock()
	{
		LLM_SCOPE(ELLMTag::FName);
		LLM_TAGSET_SCOPE_CLEAR(ELLMTagSet::Assets);
		LLM_TAGSET_SCOPE_CLEAR(ELLMTagSet::AssetClasses);
		UE_TRACE_METADATA_CLEAR_SCOPE();

#if USE_FNAME_MMAP
		if (GFNameUseMmap)
		{
#if PLATFORM_ANDROID
			if (MmapFile == -1)
			{
				FString CacheDirectoryPath = AndroidThunkCpp_GetCacheDir();
				if (!CacheDirectoryPath.IsEmpty())
				{
					auto CacheDirectoryPathCStr = StringCast<char>(*CacheDirectoryPath);
					MmapFile = open(CacheDirectoryPathCStr.Get(), O_TMPFILE | O_EXCL | O_RDWR, S_IRUSR | S_IWUSR);
				}
			}
#else
			if (MmapFile == -1)
			{
				MmapFile = fileno(tmpfile());
			}
#endif

			if (MmapFile != -1)
			{
				// Increase file size, write a single 0 at the end
				char Zero = 0;
				pwrite(MmapFile, &Zero, 1, MmapFileOffset + PageAlignedBlockSizeBytes - 1);

				if (CurrentMmapAddress == nullptr || CurrentMmapOffset >= MmapSizeBytes)
				{
					void* AddressHint = NULL;
					if (CurrentMmapAddress != nullptr)
					{
						AddressHint = CurrentMmapAddress + MmapSizeBytes;
					}

					CurrentMmapAddress = (uint8*)mmap(AddressHint, MmapSizeBytes, PROT_NONE, MAP_SHARED, MmapFile, MmapFileOffset);

					
					CurrentMmapOffset = 0;
					MmappedAddresses.Add(CurrentMmapAddress);
				}

				MmapFileOffset += PageAlignedBlockSizeBytes;
				
				uint8* MappedPointer = CurrentMmapAddress + CurrentMmapOffset;
				CurrentMmapOffset += PageAlignedBlockSizeBytes;

				// enable read/write for the new block
				mprotect(MappedPointer, PageAlignedBlockSizeBytes, PROT_READ | PROT_WRITE);

				return MappedPointer;
			}
		}
#endif
		return (uint8*)FMemory::Malloc(BlockSizeBytes, Stride);
	}
	
	void AllocateNewBlock()
	{
		// Null-terminate final entry to allow DebugDump() entry iteration
#if UE_FNAME_OUTLINE_NUMBER
		if (CurrentByteCursor + NumberedEntrySize <= BlockSizeBytes)
		{
			FNameEntry* Terminator = (FNameEntry*)(Blocks[CurrentBlock] + CurrentByteCursor);
			Terminator->Header.Len = 0;
			Terminator->NumberedName = {}; // Might be unaligned but only DebugDumpBlock will read it
		}
#else
		if (CurrentByteCursor + FNameEntry::GetDataOffset() <= BlockSizeBytes)
		{
			FNameEntry* Terminator = (FNameEntry*)(Blocks[CurrentBlock] + CurrentByteCursor);
			Terminator->Header.Len = 0;
		}
#endif

		if (CurrentByteCursor < BlockSizeBytes)
		{
			Waste += (BlockSizeBytes - CurrentByteCursor);
		}

		++CurrentBlock;
		CurrentByteCursor = 0;

		checkf(CurrentBlock < FNameMaxBlocks, TEXT("FName overflow, allocated %dMB of string data. "
			"FName strings are never freed and should be created sparingly. "
			"Some system might be generating too many FNames, see call stack. "), FNameMaxBlocks * BlockSizeBytes >> 20);

		// Allocate block unless it's already reserved
		if (Blocks[CurrentBlock] == nullptr)
		{
			Blocks[CurrentBlock] = AllocBlock();
		}

		if (!GFNameUseMmap)
		{
			FPlatformMisc::Prefetch(Blocks[CurrentBlock]);
		}
	}

	mutable FRWLock Lock;
	TSAN_ATOMIC(uint32) CurrentBlock = 0;
	uint32 CurrentByteCursor = 0;
	TSAN_ATOMIC(uint8*) Blocks[FNameMaxBlocks] = {};
	TSAN_ATOMIC(int64) Waste = 0; 
#if USE_FNAME_MMAP
	TArray<uint8*> MmappedAddresses;
	int32 MmapFile = -1;
	int32 MmapFileOffset = 0;
	int32 CurrentMmapOffset = 0;
	uint8* CurrentMmapAddress = nullptr;
#endif
};

// Increasing shards reduces contention but uses more memory and adds cache pressure.
// Reducing contention matters when multiple threads create FNames in parallel.
// Contention exists in some tool scenarios, for instance between main thread
// and asset data gatherer thread during editor startup.
// At runtime the async loading thread contends with game thread during asset loading.
#if defined(UE_FNAME_POOL_SHARD_COUNT_OVERRIDE)
static constexpr uint32 FNamePoolShards = UE_FNAME_POOL_SHARD_COUNT_OVERRIDE;
static constexpr uint32 FNamePoolShardBits = FPlatformMath::ConstExprCeilLogTwo(FNamePoolShards);
#else
#if WITH_CASE_PRESERVING_NAME
static constexpr uint32 FNamePoolShardBits = 10;
#else
static constexpr uint32 FNamePoolShardBits = 8;
#endif
static constexpr uint32 FNamePoolShards = 1U << FNamePoolShardBits;
#endif

#if !defined(UE_FNAME_POOL_INITIAL_SLOT_BITS)
#define UE_FNAME_POOL_INITIAL_SLOT_BITS 8
#endif

constexpr uint32 FNamePoolInitialSlotBits = UE_FNAME_POOL_INITIAL_SLOT_BITS;
constexpr uint32 FNamePoolInitialSlotsPerShard = 1 << FNamePoolInitialSlotBits;

/** Hashes name into 64 bits that determines shard and slot index.
 *	
 *	A small part of the hash is also stored in unused bits of the slot and entry. 
 *	The former optimizes linear probing by accessing less entry data.
 *	The latter optimizes linear probing by avoiding copying and deobfuscating entry data.
 *
 *	The slot index could be stored in the slot, at least in non shipping / test configs.
 *	This costs memory by doubling slot size but would essentially never touch entry data
 *	nor copy and deobfuscate a name needlessy. It also allows growing the hash table
 *	without rehashing the strings, since the unmasked slot index would be known.
 */
struct FNameHash
{
	uint32 ShardIndex;
	uint32 UnmaskedSlotIndex; // Determines at what slot index to start probing
	uint32 SlotProbeHash; // Helps cull equality checks (decode + strnicmp) when probing slots
	FNameEntryHeader EntryProbeHeader; // Helps cull equality checks when probing inspects entries

	static constexpr uint64 AlgorithmId = 0xC1640000;
	static constexpr uint32 ShardMask = (1U << FNamePoolShardBits) - 1U;
	static_assert((ShardMask & FNameSlot::ProbeHashMask) == 0, "Probe hash mask overlaps with bits used to calculate shard index");

	static uint32 GetShardIndexFromHiBits(uint32 HiBits)
	{
		return HiBits % FNamePoolShards;
	}

	static uint32 GetShardIndex(uint64 Hash)
	{
		return GetShardIndexFromHiBits(static_cast<uint32>(Hash >> 32));
	}

	template<class CharType>
	static uint64 GenerateHash(const CharType* Str, int32 Len)
	{
		return CityHash64(reinterpret_cast<const char*>(Str), Len * sizeof(CharType));
	}

	static uint64 GenerateHash(FNameEntryId StringPart, int32 NumberPart)
	{
		return CityHash128to64(Uint128_64(NumberPart, StringPart.ToUnstableInt()));
	}

	template<class CharType>
	FNameHash(const CharType* Str, int32 Len)
		: FNameHash(GenerateHash(Str, Len), Len, (bool)IsAnsiNone(Str, Len), sizeof(CharType) == sizeof(WIDECHAR))
	{}

	template<class CharType>
	FNameHash(const CharType* Str, int32 Len, uint64 Hash)
		: FNameHash(Hash, Len, (bool)IsAnsiNone(Str, Len), sizeof(CharType) == sizeof(WIDECHAR))
	{
	}

	FNameHash(FNameEntryId StringPart, int32 NumberPart)
		: FNameHash(GenerateHash(StringPart, NumberPart), 0, false, false)
	{
	}

	FNameHash(ENoInit)
	{

	}
		
	FNameHash(uint64 Hash, int32 Len, bool IsNone, bool bIsWide)
	{
		uint32 Hi = static_cast<uint32>(Hash >> 32);
		uint32 Lo = static_cast<uint32>(Hash);

		// "None" has FNameEntryId with a value of zero
		// Always set a bit in SlotProbeHash for "None" to distinguish unused slot values from None
		// @see FNameSlot::Used()
		uint32 IsNoneBit = IsNone << FNameSlot::ProbeHashShift;

		ShardIndex = GetShardIndexFromHiBits(Hi);
		UnmaskedSlotIndex = Lo;
		SlotProbeHash = (Hi & FNameSlot::ProbeHashMask) | IsNoneBit;
		EntryProbeHeader.Len = Len;
		EntryProbeHeader.bIsWide = bIsWide;

		// When we always use lowercase hashing, we can store parts of the hash in the entry
		// to avoid copying and decoding entries needlessly.
#if !WITH_CASE_PRESERVING_NAME		
		static constexpr uint32 EntryProbeMask = (1u << FNameEntryHeader::ProbeHashBits) - 1; 
		EntryProbeHeader.LowercaseProbeHash = static_cast<uint16>((Hi >> FNamePoolShardBits) & EntryProbeMask);
#endif
	}
	
	uint32 GetProbeStart(uint32 SlotMask) const
	{
		return UnmaskedSlotIndex & SlotMask;
	}

	static uint32 GetProbeStart(uint32 UnmaskedSlotIndex, uint32 SlotMask)
	{
		return UnmaskedSlotIndex & SlotMask;
	}

	static uint32 IsAnsiNone(const WIDECHAR* Str, int32 Len)
	{
		return 0;
	}

	static uint32 IsAnsiNone(const ANSICHAR* Str, int32 Len)
	{
		if (Len != 4)
		{
			return 0;
		}

#if PLATFORM_LITTLE_ENDIAN
		static constexpr uint32 NoneAsInt = 0x454e4f4e;
#else
		static constexpr uint32 NoneAsInt = 0x4e4f4e45;
#endif
		static constexpr uint32 ToUpperMask = 0xdfdfdfdf;

		uint32 FourChars = FPlatformMemory::ReadUnaligned<uint32>(Str);
		return (FourChars & ToUpperMask) == NoneAsInt;
	}

	bool operator==(const FNameHash& Rhs) const
	{
		return  ShardIndex == Rhs.ShardIndex &&
				UnmaskedSlotIndex == Rhs.UnmaskedSlotIndex &&
				SlotProbeHash == Rhs.SlotProbeHash &&
				EntryProbeHeader == Rhs.EntryProbeHeader;
	}
};

template<class CharType>
FORCENOINLINE uint64 GenerateLowerCaseHash(const CharType* Str, uint32 Len)
{
	CharType LowerStr[NAME_SIZE];
	for (uint32 I = 0; I < Len; ++I)
	{
		LowerStr[I] = TChar<CharType>::ToLower(Str[I]);
	}

	return FNameHash::GenerateHash(LowerStr, Len);
}

static uint64 GenerateLowerCaseHash(FNameStringView Name)
{
	return Name.bIsWide ? GenerateLowerCaseHash(Name.Wide, Name.Len) : GenerateLowerCaseHash(Name.Ansi, Name.Len);
}

template<class CharType>
FORCENOINLINE FNameHash HashLowerCase(const CharType* Str, uint32 Len)
{
	CharType LowerStr[NAME_SIZE];
	UE_CLOGF(Len > NAME_SIZE, LogCore, Fatal,
		"FName length is too long; HashLowerCase value is undefined. Length = %d, MaxLength = %d, Name=%.*ls...",
		Len, NAME_SIZE, NAME_SIZE, StringCast<TCHAR>(Str, NAME_SIZE).Get());
	for (uint32 I = 0; I < Len; ++I)
	{
		LowerStr[I] = TChar<CharType>::ToLower(Str[I]);
	}
	return FNameHash(LowerStr, Len);
}

template<ENameCase Sensitivity>
FNameHash HashName(FNameStringView Name);

template<>
FNameHash HashName<ENameCase::IgnoreCase>(FNameStringView Name)
{
	return Name.IsAnsi() ? HashLowerCase(Name.Ansi, Name.Len) : HashLowerCase(Name.Wide, Name.Len);
}
template<>
FNameHash HashName<ENameCase::CaseSensitive>(FNameStringView Name)
{
	return Name.IsAnsi() ? FNameHash(Name.Ansi, Name.Len) : FNameHash(Name.Wide, Name.Len);
}

bool FNameStringView::IsNoneString() const
{ 
	return bool(bIsWide ? FNameHash::IsAnsiNone(Wide, Len) : FNameHash::IsAnsiNone(Ansi, Len));
}

template<ENameCase Sensitivity>
struct FNameValue
{
	explicit FNameValue(FNameStringView InName)
		: Name(InName)
		, Hash(HashName<Sensitivity>(InName))
	{}

	FNameValue(FNameStringView InName, FNameHash InHash)
		: Name(InName)
		, Hash(InHash)
	{}

	FNameValue(FNameStringView InName, uint64 InHash)
	: Name(InName)
	, Hash(Name.bIsWide ? FNameHash(Name.Wide, Name.Len, InHash) : FNameHash(Name.Ansi, Name.Len, InHash))
	{}

	FNameStringView Name;
	FNameHash Hash;
#if WITH_CASE_PRESERVING_NAME
	FNameEntryId ComparisonId;
#endif
};

#if UE_FNAME_OUTLINE_NUMBER
template<ENameCase Sensitivity>
struct FNumberedNameValue
{
	FNumberedNameValue()
		: Hash(NoInit)
	{
	}

	FNumberedNameValue(FNameEntryId InStringPart, int32 InNumberPart)
		: StringPart(InStringPart)
		, NumberPart(InNumberPart)
		, Hash(InStringPart, NumberPart)
	{
	}

	FNameEntryId	StringPart;
	int32			NumberPart;
	FNameHash		Hash;

#if WITH_CASE_PRESERVING_NAME
	FNameEntryId ComparisonId;
#endif
};

using FNumberedNameComparisonValue = FNumberedNameValue<ENameCase::IgnoreCase>;
#if WITH_CASE_PRESERVING_NAME
using FNumberedNameDisplayValue = FNumberedNameValue<ENameCase::CaseSensitive>;
#endif
#endif

using FNameComparisonValue = FNameValue<ENameCase::IgnoreCase>;
#if WITH_CASE_PRESERVING_NAME
using FNameDisplayValue = FNameValue<ENameCase::CaseSensitive>;
#endif

FORCEINLINE TOptional<FNameEntryId> GetExistingComparisonId(const FNameComparisonValue& Value)  { return TOptional<FNameEntryId>(); }
#if WITH_CASE_PRESERVING_NAME
FORCEINLINE TOptional<FNameEntryId> GetExistingComparisonId(const FNameDisplayValue& Value)		{ return Value.ComparisonId; }
#endif

#if UE_FNAME_OUTLINE_NUMBER
FORCEINLINE TOptional<FNameEntryId> GetExistingComparisonId(const FNumberedNameComparisonValue& Value) { return TOptional<FNameEntryId>(); }
#if WITH_CASE_PRESERVING_NAME
FORCEINLINE TOptional<FNameEntryId> GetExistingComparisonId(const FNumberedNameDisplayValue& Value) { return Value.ComparisonId; }
#endif // WITH_CASE_PRESERVING_NAME
#endif // UE_FNAME_OUTLINE_NUMBER

// For pre-locked batch operations
struct FNullScopeLock
{
	FNullScopeLock(FRWLock&) {}
};

// One name to be loaded in a large batch
template<ENameCase Sensitivity>
struct FNameLoad
{
	FNameValue<Sensitivity> In;
	FDisplayNameEntryId* Out = nullptr;
	bool bInReuseComparisonEntry = false;
	bool bOutCreatedNewEntry = false;

	void SetOut(FNameEntryId Id);
};

using FNameComparisonLoad = FNameLoad<ENameCase::IgnoreCase>;
template<> void FNameComparisonLoad::SetOut(FNameEntryId Id) { Out->SetLoadedComparisonId(Id); }
#if WITH_CASE_PRESERVING_NAME
using FNameDisplayLoad = FNameLoad<ENameCase::CaseSensitive>;
template<> void FNameDisplayLoad::SetOut(FNameEntryId Id) { Out->SetLoadedDifferentDisplayId(Id); }
#endif

class alignas(PLATFORM_CACHE_LINE_SIZE) FNamePoolShardBase : FNoncopyable
{
public:
	void Initialize(FNameEntryAllocator& InEntries)
	{
		LLM_SCOPE(ELLMTag::FName);
		LLM_TAGSET_SCOPE_CLEAR(ELLMTagSet::Assets);
		LLM_TAGSET_SCOPE_CLEAR(ELLMTagSet::AssetClasses);
		UE_TRACE_METADATA_CLEAR_SCOPE();
		Entries = &InEntries;

		Slots = (FNameSlot*)FMemory::Malloc(FNamePoolInitialSlotsPerShard * sizeof(FNameSlot), alignof(FNameSlot));
		memset(Slots, 0, FNamePoolInitialSlotsPerShard * sizeof(FNameSlot));
		CapacityMask = FNamePoolInitialSlotsPerShard - 1;
	}

	// This and ~FNamePool() is not called during normal shutdown
	// but only via explicit FName::TearDown() call
	~FNamePoolShardBase()
	{
		FMemory::Free(Slots);
		UsedSlots = 0;
		CapacityMask = 0;
		Slots = nullptr;
		NumCreatedEntries = 0;
		NumCreatedWideEntries = 0;
	}

	uint32 Capacity() const	{ return CapacityMask + 1; }
	uint32 NumCreated() const { return NumCreatedEntries.load(std::memory_order_relaxed); }
	uint32 NumCreatedWide() const { return NumCreatedWideEntries.load(std::memory_order_relaxed); }
	uint32 NumCreatedWithNumber() const { return NumCreatedWithNumberEntries.load(std::memory_order_relaxed); }

protected:
	enum { LoadFactorQuotient = 9, LoadFactorDivisor = 10 }; // I.e. realloc slots when 90% full

	mutable FRWLock Lock;
	uint32 UsedSlots = 0;
	uint32 CapacityMask = 0;
	FNameSlot* Slots = nullptr;
	FNameEntryAllocator* Entries = nullptr;
	std::atomic<uint32> NumCreatedEntries{0};
	std::atomic<uint32> NumCreatedWideEntries{0};
	std::atomic<uint32> NumCreatedWithNumberEntries{0};


	template<ENameCase Sensitivity>
	FORCEINLINE static bool EntryEqualsValue(const FNameEntry& Entry, const FNameValue<Sensitivity>& Value)
	{
		// UE_FNAME_OUTLINE_NUMBER no special case handling here because entry + number entries have Len == 0 which is not true of any stored strings
		//	so the headers will compare unequal
		return Entry.Header == Value.Hash.EntryProbeHeader && EqualsSameDimensions<Sensitivity>(Entry, Value.Name);
	}

#if UE_FNAME_OUTLINE_NUMBER
	template<ENameCase Sensitivity>
	FORCEINLINE static bool EntryEqualsValue(const FNameEntry& Entry, const FNumberedNameValue<Sensitivity>& Value)
	{
		// UE_FNAME_OUTLINE_NUMBER no checking against string entries here because entry + number entries have Len == 0 which is not true of any stored strings
		//	so the headers will compare unequal
		if( Entry.Header == Value.Hash.EntryProbeHeader )
		{
			return Entry.GetNumberedName().Id == Value.StringPart && Entry.GetNumberedName().Number == Value.NumberPart;
		}
		return false;
	}

	template<ENameCase Sensitivity>
	FORCEINLINE static bool RehashNameWithNumber(const FNameEntry& Entry, FNumberedNameValue<Sensitivity>& OutValue)
	{
		if (Entry.Header.Len == 0)
		{
			OutValue = FNumberedNameValue<Sensitivity>(Entry.GetNumberedName().Id, Entry.GetNumberedName().Number);
			return true;
		}
		return false;
	}

#endif // UE_FNAME_OUTLINE_NUMBER
};

template<ENameCase Sensitivity>
class FNamePoolShard : public FNamePoolShardBase
{
public:
#if !UE_BUILD_SHIPPING 
	void LogCsvHeader(FOutputDevice& Ar) const
	{
		Ar.Logf(TEXT(",NumEntries,NumWideEntries,NumNumberedEntries,UsedSlots,Capacity,")
				TEXT("UniqueUnmaskedIndices,NumUnmaskedIndexCollisions,MaxUnmaskedIndexCollisions,")
				TEXT("UniqueMaskedIndices,NumMaskedIndexCollisions,MaxMaskedIndexCollisions,")
				TEXT("UniqueProbeHashes,NumProbeHashCollisions,MaxProbeHashCollisions")
				);
	}

	void LogStatsCsv(FOutputDevice& Ar) const
	{
		FRWScopeLock _(Lock, FRWScopeLockType::SLT_ReadOnly);

		TMap<uint32, int32> CountByUnmaskedIndex;
		TMap<uint32, int32> CountByMaskedIndex;
		TMap<uint32, int32> CountBySlotProbeHash;

		for (uint32 i = 0; i < Capacity(); ++i)
		{
			FNameSlot& Slot = Slots[i];
			if (!Slot.Used()) 
			{
				continue; 
			}

			const FNameEntry& Entry = Entries->Resolve(Slot.GetId());
#if UE_FNAME_OUTLINE_NUMBER
			FNumberedNameValue<Sensitivity> Value;
			if (RehashNameWithNumber(Entry, Value))
			{
				CountByUnmaskedIndex.FindOrAdd(Value.Hash.UnmaskedSlotIndex)++;
				CountByMaskedIndex.FindOrAdd(FNameHash::GetProbeStart(Value.Hash.UnmaskedSlotIndex, CapacityMask))++;
				CountBySlotProbeHash.FindOrAdd(Value.Hash.SlotProbeHash)++;
			}
			else
#endif
			{
				FNameBuffer DecodeBuffer;
				FNameStringView Name = Entry.MakeView(DecodeBuffer);
				FNameHash Hash = HashName<Sensitivity>(Name);
				CountByUnmaskedIndex.FindOrAdd(Hash.UnmaskedSlotIndex)++;
				CountByMaskedIndex.FindOrAdd(FNameHash::GetProbeStart(Hash.UnmaskedSlotIndex, CapacityMask))++;
				CountBySlotProbeHash.FindOrAdd(Hash.SlotProbeHash)++;
			}
		}

		CountByUnmaskedIndex.ValueSort(TGreater<int32>());
		CountByMaskedIndex.ValueSort(TGreater<int32>());
		CountBySlotProbeHash.ValueSort(TGreater<int32>());


		const uint32 UniqueUnmaskedIndices = CountByUnmaskedIndex.Num();
		const uint32 UniqueMaskedIndices = CountByMaskedIndex.Num();
		const uint32 UniqueProbeHashes = CountBySlotProbeHash.Num();

		int32 MaxUnmaskedIndexCollisions = MIN_int32;
		int32 NumUnmaskedIndexCollisions = 0;
		for (const TPair<uint32, int32>& Pair : CountByUnmaskedIndex)
		{
			NumUnmaskedIndexCollisions += Pair.Value - 1;
			MaxUnmaskedIndexCollisions = FMath::Max(MaxUnmaskedIndexCollisions, Pair.Value);
		}

		int32 MaxMaskedIndexCollisions = MIN_int32;
		int32 NumMaskedIndexCollisions = 0;
		for (const TPair<uint32, int32>& Pair : CountByMaskedIndex)
		{
			NumMaskedIndexCollisions += Pair.Value - 1;
			MaxMaskedIndexCollisions = FMath::Max(MaxMaskedIndexCollisions, Pair.Value);
		}

		int32 MaxProbeHashCollisions = MIN_int32;
		int32 NumProbeHashCollisions = 0;
		for (const TPair<uint32, int32>& Pair : CountBySlotProbeHash)
		{
			NumProbeHashCollisions += Pair.Value - 1;
			MaxProbeHashCollisions = FMath::Max(MaxProbeHashCollisions, Pair.Value);
		}

		Ar.Logf(TEXT(",%u,%u,%u,%u,%u,%u,%d,%d,%u,%d,%d,%u,%d,%d"),
			NumCreated(), NumCreatedWide(), NumCreatedWithNumber(), UsedSlots, Capacity(),
			UniqueUnmaskedIndices, NumUnmaskedIndexCollisions, MaxUnmaskedIndexCollisions,
			UniqueMaskedIndices, NumMaskedIndexCollisions, MaxMaskedIndexCollisions,
			UniqueProbeHashes, NumProbeHashCollisions, MaxProbeHashCollisions
		);
	}
#endif

	FNameEntryId Find(const FNameValue<Sensitivity>& Value) const
	{
		FNameEntryId Result;
		UE_AUTORTFM_OPEN
		{
			FRWScopeLock _(Lock, FRWScopeLockType::SLT_ReadOnly);

			FNameSlot& Slot = Probe(Value);
			Result = Slot.GetId();
		};
		return Result;
	}

	template<class ScopeLock = FWriteScopeLock>
	FORCEINLINE FNameEntryId Insert(const FNameValue<Sensitivity>& Value, bool& bCreatedNewEntry)
	{
		ScopeLock _(Lock);
		FNameSlot& Slot = Probe(Value);

		if (Slot.Used())
		{
			return Slot.GetId();
		}

		bCreatedNewEntry = true;
		return CreateAndInsertEntry<ScopeLock>(Slot, Value);
	}

	void InsertExistingEntry(FNameHash Hash, FNameEntryId ExistingId)
	{
		InsertExistingEntryImpl<FWriteScopeLock>(Hash, ExistingId);
	}

#if UE_FNAME_OUTLINE_NUMBER
	FNameEntryId FindWithNumber(const FNumberedNameValue<Sensitivity>& Value) const
	{
		FRWScopeLock _(Lock, FRWScopeLockType::SLT_ReadOnly);

		FNameSlot& Slot = ProbeWithNumber(Value);
		return Slot.GetId();
	}

	template<class ScopeLock = FWriteScopeLock>
	FNameEntryId InsertWithNumber(const FNumberedNameValue<Sensitivity>& Value, bool& bCreatedNewEntry)
	{
		ScopeLock _(Lock);
		FNameSlot& Slot = ProbeWithNumber(Value);

		if (Slot.Used())
		{
			return Slot.GetId();
		}

		bCreatedNewEntry = true;
		return CreateAndInsertEntryWithNumber<ScopeLock>(Slot, Value);
	}

#if WITH_CASE_PRESERVING_NAME
	void InsertExistingEntryWithNumber(FNumberedNameValue<Sensitivity> Value)
	{
		InsertExistingEntryWithNumberImpl<FWriteScopeLock>(Value);
	}
#endif // WITH_CASE_PRESERVING_NAME
#endif // UE_FNAME_OUTLINE_NUMBER

private:
	template<class ShardScopeLock>
	void InsertExistingEntryImpl(FNameHash Hash, FNameEntryId ExistingId)
	{
		FNameSlot NewLookup(ExistingId, Hash.SlotProbeHash);

		ShardScopeLock _(Lock);
		 
		FNameSlot& Slot = Probe(Hash.UnmaskedSlotIndex, [=](FNameSlot Old) { return Old == NewLookup; });
		if (!Slot.Used())
		{
			ClaimSlot(Slot, NewLookup);
		}
	}

#if UE_FNAME_OUTLINE_NUMBER && WITH_CASE_PRESERVING_NAME
	template<class ShardScopeLock>
	void InsertExistingEntryWithNumberImpl(FNumberedNameDisplayValue Value)
	{
		FNameSlot NewLookup(Value.ComparisonId, Value.Hash.SlotProbeHash);

		ShardScopeLock _(Lock);

		FNameSlot& Slot = ProbeWithNumber(Value);

		if (!Slot.Used())
		{
			ClaimSlot(Slot, NewLookup);
		}
	}
#endif

public:
	void InsertBatch(const TArrayView<FNameLoad<Sensitivity>> Batch)
	{
		if (Batch.Num() <= 0)
		{
			return;
		}

		uint32 ProbeLookAhead = FMath::Min(Batch.Num() - 1, 4);

		FWriteScopeLock _(Lock);
		
		// Prefetch first items and prepare an iterator that's ProbeLookAhead items ahead
		for (uint32 BatchIdx = 0; BatchIdx < ProbeLookAhead; ++BatchIdx)
		{
			ProbePrefetch(Batch[BatchIdx].In);
		}
		const FNameLoad<Sensitivity>* PrefetchIt = Batch.GetData() + ProbeLookAhead;

		// Resolve existing entries
		uint32 NumNewSlots = 0;
		for (FNameLoad<Sensitivity>& Request : Batch)
		{
			// Note that this is compiled out for ENameCase::IgnoreCase
			if (Sensitivity == ENameCase::CaseSensitive && Request.bInReuseComparisonEntry)
			{
				// Reuse comparison id as display id
				check(*Request.Out == GetExistingComparisonId(Request.In).GetValue());
				Request.bOutCreatedNewEntry = false;
				++NumNewSlots;
			}
			else
			{
				// Prefetch next item and increment iterator unless we're at the last item.
				// Prefetching the last item a few times is much faster than branching.
				ProbePrefetch(PrefetchIt->In);
				PrefetchIt += PrefetchIt != &Batch.Last();
				
				// Probe and write Request.Out without branching.
				// It'll be overwritten below if bOutCreatedNewEntry is false.
				FNameSlot Slot = Probe(Request.In);
				Request.SetOut(Slot.GetId());
				Request.bOutCreatedNewEntry = !Slot.Used();
				NumNewSlots += !Slot.Used();
			}
		}

		// Create and insert new entries
		if (NumNewSlots > 0)
		{
			ReserveImpl<FNullScopeLock>(UsedSlots + NumNewSlots);

			Entries->BatchLock();
			for (FNameLoad<Sensitivity>& Request : Batch)
			{
				FPlatformMisc::Prefetch(Request.In.Name.Data);

				if (Sensitivity == ENameCase::CaseSensitive && Request.bInReuseComparisonEntry)
				{
					InsertExistingEntryImpl<FNullScopeLock>(Request.In.Hash, GetExistingComparisonId(Request.In).GetValue());
				}
				else if (Request.bOutCreatedNewEntry)
				{
					// Check Slot.Used() since batch may contain duplicates
					FNameSlot& Slot = Probe(Request.In);
					Request.bOutCreatedNewEntry = !Slot.Used();
					Request.SetOut(Slot.Used() ? Slot.GetId() : CreateAndInsertEntry<FNullScopeLock>(Slot, Request.In));
				}
			}
			Entries->BatchUnlock();
		}
	}

	void Reserve(uint32 Num)
	{
		ReserveImpl<FWriteScopeLock>(Num);
	}

private:
	template<class ShardScopeLock>
	void ReserveImpl(uint32 Num)
	{
		// Num + 1 to compensate for division rounding down
		uint32 WantedCapacity = FMath::RoundUpToPowerOfTwo((Num + 1) * LoadFactorDivisor / LoadFactorQuotient);

		ShardScopeLock _(Lock);
		if (WantedCapacity > Capacity())
		{
			Grow(WantedCapacity);
		}
	}

	void ClaimSlot(FNameSlot& UnusedSlot, FNameSlot NewValue)
	{
		checkSlow(!UnusedSlot.Used());

		UnusedSlot = NewValue;

		++UsedSlots;
		if (UsedSlots * LoadFactorDivisor > LoadFactorQuotient * Capacity())
		{
			Grow();
		}
	}

	template<class EntryScopeLock>
	FNameEntryId CreateAndInsertEntry(FNameSlot& Slot, const FNameValue<Sensitivity>& Value)
	{
		FNameEntryId NewEntryId = Entries->Create<EntryScopeLock>(Value.Name, GetExistingComparisonId(Value), Value.Hash.EntryProbeHeader);

		ClaimSlot(Slot, FNameSlot(NewEntryId, Value.Hash.SlotProbeHash));

		NumCreatedEntries.fetch_add(1, std::memory_order_relaxed);
		NumCreatedWideEntries.fetch_add(Value.Name.bIsWide, std::memory_order_relaxed);
		
		return NewEntryId;
	}

#if UE_FNAME_OUTLINE_NUMBER
	template<class EntryScopeLock>
	FNameEntryId CreateAndInsertEntryWithNumber(FNameSlot& Slot, const FNumberedNameValue<Sensitivity>& Value)
	{
		FNameEntryId NewEntryId = Entries->CreateWithNumber<EntryScopeLock>(Value.StringPart, Value.NumberPart, GetExistingComparisonId(Value), Value.Hash.EntryProbeHeader);

		ClaimSlot(Slot, FNameSlot(NewEntryId, Value.Hash.SlotProbeHash));

		NumCreatedWithNumberEntries.fetch_add(1, std::memory_order_relaxed);;

		return NewEntryId;
	}
#endif // UE_FNAME_OUTLINE_NUMBER

	void Grow()
	{
		Grow(Capacity() * 2);
	}

	void Grow(const uint32 NewCapacity)
	{
		LLM_SCOPE(ELLMTag::FName);
		LLM_TAGSET_SCOPE_CLEAR(ELLMTagSet::Assets);
		LLM_TAGSET_SCOPE_CLEAR(ELLMTagSet::AssetClasses);
		UE_TRACE_METADATA_CLEAR_SCOPE();
		TArrayView<FNameSlot> OldSlots(Slots, Capacity());
		const uint32 OldUsedSlots = UsedSlots;

		Slots = (FNameSlot*)FMemory::Malloc(NewCapacity * sizeof(FNameSlot), alignof(FNameSlot));
		memset(Slots, 0, NewCapacity * sizeof(FNameSlot));
		UsedSlots = 0;
		CapacityMask = NewCapacity - 1;

		// Prefetch FNameEntry* before rehashing. Yielded 2.4x rehash speedup on a Gen5 console.
		constexpr uint32 PrefetchDepth = 8;
		FNameSlot PrefetchedSlots[PrefetchDepth];
		uint32 NumPrefetched = 0;
		
		for (FNameSlot OldSlot : OldSlots)
		{
			if (OldSlot.Used())
			{
				FPlatformMisc::Prefetch(&Entries->Resolve(OldSlot.GetId()));
				PrefetchedSlots[NumPrefetched] = OldSlot;
		
				if (++NumPrefetched == PrefetchDepth)
				{
					for (FNameSlot PrefetchedSlot : PrefetchedSlots)
					{
						RehashAndInsert(PrefetchedSlot);
					}

					NumPrefetched = 0;
				}
			}
		}

		for (FNameSlot PrefetchedSlot : MakeArrayView(PrefetchedSlots, NumPrefetched))
		{
			RehashAndInsert(PrefetchedSlot);
		}


		check(OldUsedSlots == UsedSlots);

		FMemory::Free(OldSlots.GetData());
	}

	void ProbePrefetch(const FNameValue<Sensitivity>& Value) const
	{
		// Prefetch name data and FNameSlot*
		FPlatformMisc::Prefetch(Value.Name.Data);
		FPlatformMisc::Prefetch(Slots + FNameHash::GetProbeStart(Value.Hash.UnmaskedSlotIndex, CapacityMask));
		
		// Prefetching the FNameEntry* might help, but it involves waiting for FNameSlot* 
		// and potentially a branch if we don't want to prefetch null pointers for unused slots.
		// Would probably need two prefetch passes to see a perf increase from this and
		// I'd rather avoid even more prefetching complexity. --jtorp
	}

	/** Find slot containing value or the first free slot that should be used to store it  */
	FORCEINLINE FNameSlot& Probe(const FNameValue<Sensitivity>& Value) const
	{
		return Probe(Value.Hash.UnmaskedSlotIndex, 
			[&](FNameSlot Slot)	{ return Slot.GetProbeHash() == Value.Hash.SlotProbeHash && 
									EntryEqualsValue<Sensitivity>(Entries->Resolve(Slot.GetId()), Value); });
	}

	/** Find slot that fulfills predicate or the first free slot  */
	template<class PredicateFn>
	FORCEINLINE FNameSlot& Probe(uint32 UnmaskedSlotIndex, PredicateFn Predicate) const
	{
		const uint32 Mask = CapacityMask;
		for (uint32 I = FNameHash::GetProbeStart(UnmaskedSlotIndex, Mask); true; I = (I + 1) & Mask)
		{
			FNameSlot& Slot = Slots[I];
			if (!Slot.Used() || Predicate(Slot))
			{
				return Slot;
			}
		}
	}

#if UE_FNAME_OUTLINE_NUMBER
	/** Find slot containing value or the first free slot that should be used to store it  */
	FORCEINLINE FNameSlot& ProbeWithNumber(const FNumberedNameValue<Sensitivity>& Value) const
	{
		return Probe(Value.Hash.UnmaskedSlotIndex,
			[&](FNameSlot Slot) { return Slot.GetProbeHash() == Value.Hash.SlotProbeHash &&
			EntryEqualsValue(Entries->Resolve(Slot.GetId()), Value); });
	}
#endif // UE_FNAME_OUTLINE_NUMBER

	FORCENOINLINE // Doesn't impact performance and makes sampling profiles more informative
	void RehashAndInsert(FNameSlot OldSlot)
	{
		check(OldSlot.Used());

		const FNameEntry& Entry = Entries->Resolve(OldSlot.GetId());
#if UE_FNAME_OUTLINE_NUMBER
		FNumberedNameValue<Sensitivity> Value;
		if (RehashNameWithNumber(Entry, Value))
		{
			FNameSlot& NewSlot = Probe(Value.Hash.UnmaskedSlotIndex, [](FNameSlot Slot) { return false; });
			NewSlot = OldSlot;
			++UsedSlots;
		}
		else
#endif
		{
			FNameBuffer DecodeBuffer;
			FNameStringView Name = Entry.MakeView(DecodeBuffer);
			FNameHash Hash = HashName<Sensitivity>(Name);
			FNameSlot& NewSlot = Probe(Hash.UnmaskedSlotIndex, [](FNameSlot Slot) { return false; });
			NewSlot = OldSlot;
			++UsedSlots;
		}
	}
};


class FNamePool
{
public:
	FNamePool();
	
	void			Reserve(uint32 NumBlocks, uint32 NumEntries);
	FNameEntryId	Store(FNameStringView View);
	FNameEntryId	Find(FNameStringView View) const;
	FNameEntryId	Find(EName Ename) const;
	const EName*	FindEName(FNameEntryId Id) const;

#if UE_FNAME_OUTLINE_NUMBER
	FNameEntryId	StoreWithNumber(FNameEntryIds StringParts, int32 NumberPart);
	FNameEntryId	FindWithNumber(FNameEntryId StringPart, int32 NumberPart) const;

#if WITH_CASE_PRESERVING_NAME
	FNameEntryId	StoreValueWithNumber(FNumberedNameDisplayValue Value, bool bReuseComparisonId);
#endif // WITH_CASE_PRESERVING_NAME
#endif // UE_FNAME_OUTLINE_NUMBER

	/** @pre !!Handle */
	FNameEntry&		Resolve(FNameEntryHandle Handle) const { return Entries.Resolve(Handle); }

	bool			IsValid(FNameEntryHandle Handle) const;

	FDisplayNameEntryId	StoreValue(const FNameComparisonValue& Value);
	void			StoreBatch(uint32 ShardIdx, TArrayView<FNameComparisonLoad> Batch)	{ ComparisonShards[ShardIdx].InsertBatch(Batch); }
#if WITH_CASE_PRESERVING_NAME
	FDisplayNameEntryId	StoreValue(const FNameDisplayValue& Value, bool bReuseComparisonId);
	void			StoreBatch(uint32 ShardIdx, TArrayView<FNameDisplayLoad> Batch)		{ DisplayShards[ShardIdx].InsertBatch(Batch); }
	bool			ReuseComparisonEntry(bool bAddedComparisonEntry, const FNameDisplayValue& DisplayValue);
#endif
	/// Stats and debug related functions ///

	uint32			NumEntries() const;
	uint32			NumAnsiEntries() const;
	uint32			NumWideEntries() const;
	uint32			NumNumberedEntries() const;
	uint32			NumBlocks() const { return Entries.NumBlocks(); }
	uint32			NumSlots() const;
	void			LogStats(FOutputDevice& Ar) const;
	void			LogHashStats(FOutputDevice& Ar) const;
	uint8**			GetBlocksForDebugVisualizer() { return Entries.GetBlocksForDebugVisualizer(); }
	TArray<const FNameEntry*> DebugDump() const;

#if UE_TRACE_ENABLED
	UE::Trace::FEventRef32 Trace(const FNameEntryId& EntryId);
	UE::Trace::FEventRef32 TraceDefinition(const FNameEntryId& EntryId) const;
	void RetraceAll() const;
#endif

private:
	enum { MaxENames = 512 };

	FNameEntryAllocator Entries;

#if WITH_CASE_PRESERVING_NAME
	FNamePoolShard<ENameCase::CaseSensitive> DisplayShards[FNamePoolShards];
#endif
	FNamePoolShard<ENameCase::IgnoreCase> ComparisonShards[FNamePoolShards];

#if UE_TRACE_ENABLED
	using FTracedBitSet = TNameAtomicBitSet<FNameBlockOffsets>;
	FTracedBitSet TracedNames[FNameMaxBlocks];
#endif
	
	// Put constant lookup on separate cache line to avoid it being constantly invalidated by insertion
	alignas(PLATFORM_CACHE_LINE_SIZE) FNameEntryId ENameToEntry[(uint32)EName::MaxHardcodedNameIndex] = {};
	uint32 LargestEnameUnstableId;
	TMap<FNameEntryId, EName, TInlineSetAllocator<MaxENames>> EntryToEName;
};

FNamePool::FNamePool()
{
	for (FNamePoolShardBase& Shard : ComparisonShards)
	{
		Shard.Initialize(Entries);
	}

#if WITH_CASE_PRESERVING_NAME
	for (FNamePoolShardBase& Shard : DisplayShards)
	{
		Shard.Initialize(Entries);
	}
#endif

	// Register all hardcoded names
#define REGISTER_NAME(num, name) ENameToEntry[num] = Store(FNameStringView(#name, FCStringAnsi::Strlen(#name)));
#include "UObject/UnrealNames.inl"
#undef REGISTER_NAME

	// Make reverse mapping
	LargestEnameUnstableId = 0;
	for (uint32 ENameIndex = 0; ENameIndex < (uint32)EName::MaxHardcodedNameIndex; ++ENameIndex)
	{
		if (ENameIndex == (uint32)NAME_None || ENameToEntry[ENameIndex])
		{
			EntryToEName.Add(ENameToEntry[ENameIndex], (EName)ENameIndex);
			LargestEnameUnstableId = FMath::Max(LargestEnameUnstableId, ENameToEntry[ENameIndex].ToUnstableInt());
		}
	}

	// Verify all ENames are unique
	if (NumAnsiEntries() != EntryToEName.Num())
	{
		// we can't print out here because there may be no log yet if this happens before main starts
		if (FPlatformMisc::IsDebuggerPresent())
		{
			UE_DEBUG_BREAK();
		}
		else
		{
			FPlatformMisc::PromptForRemoteDebugging(false);
			FMessageDialog::Open(EAppMsgType::Ok, NSLOCTEXT("UnrealEd", "DuplicatedHardcodedName", "Duplicate hardcoded name"));
			FPlatformMisc::RequestExit(false, TEXT("FNamePool.DuplicateHardcodedName"));
		}
	}
}

static bool IsPureAnsi(const WIDECHAR* Str, const int32 Len)
{
	// Consider SSE version if this function takes significant amount of time
	uint32 Result = 0;
	for (int32 I = 0; I < Len; ++I)
	{
		Result |= TChar<WIDECHAR>::ToUnsigned(Str[I]);
	}
	return !(Result & 0xffffff80u);
}

FNameEntryId FNamePool::Find(EName Ename) const
{
	checkSlow(Ename < EName::MaxHardcodedNameIndex);
	return ENameToEntry[(uint32)Ename];
}

FNameEntryId FNamePool::Find(FNameStringView Name) const
{
#if WITH_CASE_PRESERVING_NAME
	// Look for an exact match with the right casing first
	FNameDisplayValue DisplayValue(Name);
	if (FNameEntryId Existing = DisplayShards[DisplayValue.Hash.ShardIndex].Find(DisplayValue))
	{
		return Existing;
	}
#endif

	FNameComparisonValue ComparisonValue(Name);
	return ComparisonShards[ComparisonValue.Hash.ShardIndex].Find(ComparisonValue);
}

FNameEntryId FNamePool::Store(FNameStringView Name)
{
#if WITH_CASE_PRESERVING_NAME
	// Look for an exact match with the right casing first
	FNameDisplayValue DisplayValue(Name);
	if (FNameEntryId Existing = DisplayShards[DisplayValue.Hash.ShardIndex].Find(DisplayValue))
	{
		return Existing;
	}
#endif

	bool bAdded = false;

	// Insert comparison name first since display value must contain comparison name
	FNameComparisonValue ComparisonValue(Name);
	FNameEntryId ComparisonId = ComparisonShards[ComparisonValue.Hash.ShardIndex].Insert(ComparisonValue, bAdded);

#if WITH_CASE_PRESERVING_NAME
	DisplayValue.ComparisonId = ComparisonId;
	return StoreValue(DisplayValue, bAdded).ToDisplayId();
#else
	return ComparisonId;
#endif
}

#if UE_FNAME_OUTLINE_NUMBER
FNameEntryId FNamePool::StoreWithNumber(FNameEntryIds StringParts, int32 NumberPart)
{
	FNameEntryId Result;

	UE_AUTORTFM_OPEN
	{
#if WITH_CASE_PRESERVING_NAME
		// Look for an exact match with the right casing first
		FNumberedNameDisplayValue DisplayValue(StringParts.DisplayId, NumberPart);
		if (FNameEntryId Existing = DisplayShards[DisplayValue.Hash.ShardIndex].FindWithNumber(DisplayValue))
		{
			Result = Existing;
		}
		else
		{
#endif
			bool bAdded = false;

			// Insert comparison name first since display value must contain comparison name
			FNumberedNameComparisonValue ComparisonValue(StringParts.ComparisonId, NumberPart);
			FNameEntryId ComparisonId = ComparisonShards[ComparisonValue.Hash.ShardIndex].InsertWithNumber(ComparisonValue, bAdded);

#if WITH_CASE_PRESERVING_NAME
			DisplayValue.ComparisonId = ComparisonId;
			Result = StoreValueWithNumber(DisplayValue, bAdded);
		}
#else
		Result = ComparisonId;
#endif
	};

	return Result;
}

FNameEntryId FNamePool::FindWithNumber(FNameEntryId StringPart, int32 NumberPart) const
{
#if WITH_CASE_PRESERVING_NAME
	// Look for an exact match with the right casing first
	FNumberedNameDisplayValue DisplayValue(StringPart, NumberPart);
	if (FNameEntryId Existing = DisplayShards[DisplayValue.Hash.ShardIndex].FindWithNumber(DisplayValue))
	{
		return Existing;
	}
	// Otherwise fall back to a name that compares equal case-insensitively
#endif

	FNumberedNameComparisonValue ComparisonValue(StringPart, NumberPart);
	return ComparisonShards[ComparisonValue.Hash.ShardIndex].FindWithNumber(ComparisonValue);
}

#if WITH_CASE_PRESERVING_NAME
FNameEntryId FNamePool::StoreValueWithNumber(FNumberedNameDisplayValue Value, bool bAddedComparisonEntry)
{
	FNamePoolShard<ENameCase::CaseSensitive>& DisplayShard = DisplayShards[Value.Hash.ShardIndex];
	if (bAddedComparisonEntry || Value.StringPart == Value.ComparisonId)
	{
		DisplayShard.InsertExistingEntryWithNumber(Value);
		return Value.ComparisonId;
	}
	else
	{
		return DisplayShard.InsertWithNumber(Value, bAddedComparisonEntry);
	}
}
#endif // WITH_CASE_PRESERVING_NAME
#endif // UE_FNAME_OUTLINE_NUMBER 

FORCEINLINE FDisplayNameEntryId FNamePool::StoreValue(const FNameComparisonValue& ComparisonValue)
{
	bool bAdded = false;
	FNameEntryId ComparisonId = ComparisonShards[ComparisonValue.Hash.ShardIndex].Insert(ComparisonValue, bAdded);

#if WITH_CASE_PRESERVING_NAME
	FNameDisplayValue DisplayValue(ComparisonValue.Name);
	DisplayValue.ComparisonId = ComparisonId;
	return StoreValue(DisplayValue, bAdded);
#else
	return FDisplayNameEntryId::FromComparisonId(ComparisonId);
#endif
}

#if WITH_CASE_PRESERVING_NAME
bool FNamePool::ReuseComparisonEntry(bool bAddedComparisonEntry, const FNameDisplayValue& DisplayValue)
{
	return bAddedComparisonEntry || EqualsSameDimensions<ENameCase::CaseSensitive>(Resolve(DisplayValue.ComparisonId), DisplayValue.Name);
}

FORCEINLINE FDisplayNameEntryId FNamePool::StoreValue(const FNameDisplayValue& DisplayValue, bool bAddedComparisonEntry)
{
	FDisplayNameEntryId Out = FDisplayNameEntryId::FromComparisonId(DisplayValue.ComparisonId);

	FNamePoolShard<ENameCase::CaseSensitive>& DisplayShard = DisplayShards[DisplayValue.Hash.ShardIndex];
	if (ReuseComparisonEntry(bAddedComparisonEntry, DisplayValue))
	{
		DisplayShard.InsertExistingEntry(DisplayValue.Hash, DisplayValue.ComparisonId);
	}
	else
	{
		Out.SetLoadedDifferentDisplayId(DisplayShard.Insert(DisplayValue, bAddedComparisonEntry));
	}

	return Out;
}
#endif

uint32 FNamePool::NumEntries() const
{
	uint32 Out = 0;
#if WITH_CASE_PRESERVING_NAME
	for (const FNamePoolShardBase& Shard : DisplayShards)
	{
		Out += Shard.NumCreated();
		Out += Shard.NumCreatedWithNumber();
	}
#endif
	for (const FNamePoolShardBase& Shard : ComparisonShards)
	{
		Out += Shard.NumCreated();
		Out += Shard.NumCreatedWithNumber();
	}

	return Out;
}

uint32 FNamePool::NumAnsiEntries() const
{
	return NumEntries() - NumWideEntries() - NumNumberedEntries();
}

uint32 FNamePool::NumWideEntries() const
{
	uint32 Out = 0;
#if WITH_CASE_PRESERVING_NAME
	for (const FNamePoolShardBase& Shard : DisplayShards)
	{
		Out += Shard.NumCreatedWide();
	}
#endif
	for (const FNamePoolShardBase& Shard : ComparisonShards)
	{
		Out += Shard.NumCreatedWide();
	}

	return Out;
}

uint32 FNamePool::NumNumberedEntries() const
{
	uint32 Out = 0;
#if UE_FNAME_OUTLINE_NUMBER
#if WITH_CASE_PRESERVING_NAME
	for (const FNamePoolShardBase& Shard : DisplayShards)
	{
		Out += Shard.NumCreatedWithNumber();
	}
#endif
	for (const FNamePoolShardBase& Shard : ComparisonShards)
	{
		Out += Shard.NumCreatedWithNumber();
	}
#endif // UE_FNAME_OUTLINE_NUMBER

	return Out;
}

uint32 FNamePool::NumSlots() const
{
	uint32 SlotCapacity = 0;
#if WITH_CASE_PRESERVING_NAME
	for (const FNamePoolShardBase& Shard : DisplayShards)
	{
		SlotCapacity += Shard.Capacity();
	}
#endif
	for (const FNamePoolShardBase& Shard : ComparisonShards)
	{
		SlotCapacity += Shard.Capacity();
	}

	return SlotCapacity;
}

#if UE_TRACE_ENABLED
UE_AUTORTFM_ALWAYS_OPEN
UE::Trace::FEventRef32 FNamePool::Trace(const FNameEntryId& EntryId)
{
	const FNameEntryHandle Handle(EntryId);
	const uint32 DefinitionId = EntryId.ToUnstableInt();
	if (!TracedNames[Handle.Block].SetBitAtomic(Handle.Offset))
	{
		return TraceDefinition(EntryId);
	}
	return UE::Trace::MakeEventRef(DefinitionId, UE_TRACE_GET_DEFINITION_TYPE_ID(Strings, FName));
}

UE::Trace::FEventRef32 FNamePool::TraceDefinition(const FNameEntryId& EntryId) const
{
	const uint32 DefinitionId = EntryId.ToUnstableInt();
	FNameEntry* DisplayEntry = &Resolve(EntryId);
	uint32 Number = 0;
 
#if UE_FNAME_OUTLINE_NUMBER
	if (DisplayEntry->IsNumbered())
	{
		Number = DisplayEntry->GetNumber();
		DisplayEntry = DisplayEntry->IsNumbered() ? &Resolve(DisplayEntry->GetNumberedName().Id) : DisplayEntry;
	}
#endif
 
	if (DisplayEntry->IsWide())
	{
		TStringBuilder<NAME_SIZE> NameString;
		DisplayEntry->AppendNameToString(NameString);
		if (Number)
		{
			NameString << TEXT("_") << NAME_INTERNAL_TO_EXTERNAL(Number);
		}
		const auto Ref = UE_TRACE_LOG_DEFINITION(Strings, FName, DefinitionId, true)
			<< FName.DisplayWide(NameString.GetData(), NameString.Len());
		return Ref;
	}
	else
	{
		TAnsiStringBuilder<NAME_SIZE> NameString;
		DisplayEntry->AppendAnsiNameToString(NameString);
		if (Number)
		{
			NameString << "_" << NAME_INTERNAL_TO_EXTERNAL(Number);
		}
		const auto Ref = UE_TRACE_LOG_DEFINITION(Strings, FName, DefinitionId, true)
			<< FName.DisplayAnsi(NameString.GetData(), NameString.Len());
		return Ref;
	}
}

void FNamePool::RetraceAll() const
{
	for (uint32 Block = 0; Block < FNameMaxBlocks; ++Block)
	{
		const FTracedBitSet& Set = TracedNames[Block];

		Set.ForEachSetBit([&](uint32 Index)
		{
			const FNameEntryHandle Handle(Block, Index);
			const FNameEntryId EntryId = Handle;
			TraceDefinition(EntryId);
		});
	}
}
#endif //UE_TRACE_ENABLED

void FNamePool::LogStats(FOutputDevice& Ar) const
{
	Ar.Logf(TEXT("%i FNames using in %iKB static memory, %.2fMB / %.2fMB in block storage, %.2fMB in lookup"), 
		NumEntries(), 
		sizeof(FNamePool) / 1024, 
		Entries.NumBlocks() * (int64)FNameEntryAllocator::BlockSizeBytes / 1024.0 / 1024.0,
		FNameMaxBlocks * (int64)FNameEntryAllocator::BlockSizeBytes / 1024.0 / 1024.0,
		NumSlots() * sizeof(FNameSlot) / 1024.0 / 1024.0
	);
	Ar.Logf(TEXT("%.2fMB of waste from aligning entries"), Entries.GetWaste() / 1024.0 / 1024.0);
	Ar.Logf(TEXT("%d ansi FNames"), NumAnsiEntries());
	Ar.Logf(TEXT("%d wide FNames"), NumWideEntries());
#if UE_FNAME_OUTLINE_NUMBER
	Ar.Logf(TEXT("%d FNames with numbers stored in entries"), NumNumberedEntries());
#endif
}

void FNamePool::LogHashStats(FOutputDevice& Ar) const
{
#if !UE_BUILD_SHIPPING
	ComparisonShards[0].LogCsvHeader(Ar);
	for (int32 i = 0; i < FNamePoolShards; ++i)
	{
		ComparisonShards[i].LogStatsCsv(Ar);
	}
#endif
}

TArray<const FNameEntry*> FNamePool::DebugDump() const
{
	TArray<const FNameEntry*> Out;
	Out.Reserve(NumEntries());
	Entries.DebugDump(Out);
	return Out;
}

bool FNamePool::IsValid(FNameEntryHandle Handle) const
{
	return Handle.Block < Entries.NumBlocks();
}

const EName* FNamePool::FindEName(FNameEntryId Id) const
{
	return Id.ToUnstableInt() > LargestEnameUnstableId ? nullptr : EntryToEName.Find(Id);
}

void FNamePool::Reserve(uint32 NumBytes, uint32 InNumEntries)
{
	uint32 NumBlocks = NumBytes / FNameEntryAllocator::BlockSizeBytes + 1;
	Entries.ReserveBlocks(NumBlocks);

	if (NumEntries() < InNumEntries)
	{
		uint32 NumEntriesPerShard = InNumEntries / FNamePoolShards + 1;

	#if WITH_CASE_PRESERVING_NAME
		for (FNamePoolShard<ENameCase::CaseSensitive>& Shard : DisplayShards)
		{
			Shard.Reserve(NumEntriesPerShard);
		}
	#endif
		for (FNamePoolShard<ENameCase::IgnoreCase>& Shard : ComparisonShards)
		{
			Shard.Reserve(NumEntriesPerShard);
		}
	}
}

static bool bNamePoolInitialized;
alignas(FNamePool) static uint8 NamePoolData[sizeof(FNamePool)];

// Only call this once per public FName function called
//
// Not using magic statics to run as little code as possible
static FNamePool& GetNamePool()
{
	if (bNamePoolInitialized)
	{
		return *(FNamePool*)NamePoolData;
	}

	FNamePool* Singleton = nullptr;
	
	UE_AUTORTFM_OPEN
	{
		Singleton = new (NamePoolData) FNamePool;
		bNamePoolInitialized = true;
		LLM(FLowLevelMemTracker::Get().FinishInitialise());
	};
	
	return *Singleton;
}

// Only call from functions guaranteed to run after FName lazy initialization
static FNamePool& GetNamePoolPostInit()
{
	checkSlow(bNamePoolInitialized);
	return (FNamePool&)NamePoolData;
}

template <ESearchCase::Type SearchCase>
static int32 CompareDifferentIdsAlphabetically(FNameEntryId AId, FNameEntryId BId)
{
	checkSlow(AId != BId);

	FNamePool& Pool = GetNamePool();
	FNameBuffer ABuffer, BBuffer;
	FNameStringView AView =	Pool.Resolve(AId).MakeView(ABuffer);
	FNameStringView BView =	Pool.Resolve(BId).MakeView(BBuffer);

	// If only one view is wide, convert the ansi view to wide as well
	if (AView.bIsWide != BView.bIsWide)
	{
		FNameStringView& AnsiView = AView.bIsWide ? BView : AView;
		FNameBuffer& AnsiBuffer =	AView.bIsWide ? BBuffer : ABuffer;
		
		FPlatformMemory::Memcpy(AnsiBuffer.AnsiName, AnsiView.Ansi, AnsiView.Len * sizeof(ANSICHAR));
        AnsiView.Ansi = AnsiBuffer.AnsiName;

		ConvertInPlace<ANSICHAR, WIDECHAR>(AnsiBuffer.AnsiName, AnsiView.Len);
		AnsiView.bIsWide = true;
	}

	int32 MinLen = FMath::Min(AView.Len, BView.Len);
	int32 StrDiff;
	if (SearchCase == ESearchCase::IgnoreCase)
	{
		StrDiff = AView.bIsWide ?
			FCStringWide::Strnicmp(AView.Wide, BView.Wide, MinLen) :
			FCStringAnsi::Strnicmp(AView.Ansi, BView.Ansi, MinLen);
	}
	else
	{
		StrDiff = AView.bIsWide ?
			FCStringWide::Strncmp(AView.Wide, BView.Wide, MinLen) :
			FCStringAnsi::Strncmp(AView.Ansi, BView.Ansi, MinLen);
	}
	if (StrDiff) return StrDiff;

	return AView.Len - BView.Len;
}

int32 FNameEntryId::CompareLexical(FNameEntryId Rhs) const
{
	return Value != Rhs.Value ? CompareDifferentIdsAlphabetically<ESearchCase::IgnoreCase>(*this, Rhs) : 0;
}

int32 FNameEntryId::CompareLexicalSensitive(FNameEntryId Rhs) const
{
	return Value != Rhs.Value ? CompareDifferentIdsAlphabetically<ESearchCase::CaseSensitive>(*this, Rhs) : 0;
}

#if !UE_BUILD_SHIPPING && !UE_BUILD_TEST
	void CallNameCreationHook();
#else
	FORCEINLINE void CallNameCreationHook()
	{
	}
#endif

static FNameEntryId DebugCastNameEntryId(int32 Id) { return (FNameEntryId&)(Id); }

/**
* Helper function that can be used inside the debuggers watch window. E.g. "DebugFName(Class->Name.Index)". 
*
* @param	Index	Name index to look up string for
* @return			Associated name
*/
const TCHAR* DebugFName(FNameEntryId Index)
{
	// Hardcoded static array. This function is only used inside the debugger so it should be fine to return it.
	static TCHAR TempName[NAME_SIZE];
	FCString::Strcpy(TempName, *FName::SafeString(Index));
	return TempName;
}

/**
* Helper function that can be used inside the debuggers watch window. E.g. "DebugFName(Class->Name.Index, Class->Name.Number)". 
*
* @param	Index	Name index to look up string for
* @param	Number	Internal instance number of the FName to print (which is 1 more than the printed number)
* @return			Associated name
*/
const TCHAR* DebugFName(int32 Index, int32 Number)
{
	// Hardcoded static array. This function is only used inside the debugger so it should be fine to return it.
	static TCHAR TempName[FName::StringBufferSize];
	FCString::Strcpy(TempName, *FName::SafeString(DebugCastNameEntryId(Index), Number));
	return TempName;
}

/**
 * Helper function that can be used inside the debuggers watch window. E.g. "DebugFName(Class->Name)". 
 *
 * @param	Name	Name to look up string for
 * @return			Associated name
 */
const TCHAR* DebugFName(FName& Name)
{
	// Hardcoded static array. This function is only used inside the debugger so it should be fine to return it.
	static TCHAR TempName[FName::StringBufferSize];
	FCString::Strcpy(TempName, *FName::SafeString(Name.GetDisplayIndex(), Name.GetNumber()));
	return TempName;
}

template <typename TCharType>
static uint16 GetRawCasePreservingHash(const TCharType* Source)
{
	return FCrc::StrCrc32(Source) & 0xFFFF;

}
template <typename TCharType>
static uint16 GetRawNonCasePreservingHash(const TCharType* Source)
{
	return FCrc::Strihash_DEPRECATED(Source) & 0xFFFF;
}

/*-----------------------------------------------------------------------------
	FNameEntry
-----------------------------------------------------------------------------*/

// Make Clang keep FNameEntry type debug info needed for debug visualizers
FNameEntry::FNameEntry(FClangKeepDebugInfo) {}

void FNameEntry::CopyUnterminatedName(ANSICHAR* Out, uint32 Len) const
{
	FNameStore::Copy(Out, NameData, Len);
}

void FNameEntry::CopyUnterminatedName(WIDECHAR* Out, uint32 Len) const
{
	FNameStore::Copy(Out, NameData, Len);
}

FORCEINLINE const WIDECHAR* FNameEntry::GetUnterminatedName(WIDECHAR(&OptionalDecodeBuffer)[NAME_SIZE]) const
{
	if (const WIDECHAR* StoredPtr = FNameStore::TryGetWide(NameData))
	{
		return StoredPtr;
	}
	FNameStore::Copy(OptionalDecodeBuffer, NameData, Header.Len);
	return OptionalDecodeBuffer;
}

FORCEINLINE ANSICHAR const* FNameEntry::GetUnterminatedName(ANSICHAR(&OptionalDecodeBuffer)[NAME_SIZE]) const
{
	if (const ANSICHAR* StoredPtr = FNameStore::TryGetAnsi(NameData))
	{
		return StoredPtr;
	}
	
	FNameStore::Copy(OptionalDecodeBuffer, NameData, Header.Len);
	return OptionalDecodeBuffer;
}

int32 FNameEntry::GetNameLengthUtf8() const
{
	if (Header.bIsWide)
	{
		FNameBuffer Temp;
		return FPlatformString::ConvertedLength<UTF8CHAR>(GetUnterminatedName(Temp.WideName), Header.Len);
	}
	else
	{
		return Header.Len;
	}
}

FORCEINLINE FNameStringView FNameEntry::MakeView(FNameBuffer& OptionalDecodeBuffer) const
{
#if UE_FNAME_OUTLINE_NUMBER
	checkf(Header.Len != 0, TEXT("Cannot make a string view for a name-with-number entry"));
#else // UE_FNAME_OUTLINE_NUMBER
	check(Header.Len != 0);
#endif // UE_FNAME_OUTLINE_NUMBER
	return IsWide()	? FNameStringView(GetUnterminatedName(OptionalDecodeBuffer.WideName), GetNameLength())
					: FNameStringView(GetUnterminatedName(OptionalDecodeBuffer.AnsiName), GetNameLength());
}

void FNameEntry::GetUnterminatedName(TCHAR* OutName, uint32 OutLen) const
{
	CopyAndConvertUnterminatedName(OutName, FMath::Min<uint32>(Header.Len, OutLen));
}

void FNameEntry::GetName(TCHAR(&OutName)[NAME_SIZE]) const
{
	CopyAndConvertUnterminatedName(OutName, Header.Len);
	OutName[GetNameLength()] = TEXT('\0');
}

void FNameEntry::CopyAndConvertUnterminatedName(TCHAR* OutName, uint32 Len) const
{
	if (sizeof(TCHAR) < sizeof(WIDECHAR) && IsWide()) // Normally compiled out
	{
		FNameBuffer Temp;
		CopyUnterminatedName(Temp.WideName, Len);
		ConvertInPlace<WIDECHAR, TCHAR>(Temp.WideName, Len);
		FPlatformMemory::Memcpy(OutName, Temp.AnsiName, Len * sizeof(TCHAR));
	}
	else if (IsWide())
	{
		CopyUnterminatedName((WIDECHAR*)OutName, Len);
		ConvertInPlace<WIDECHAR, TCHAR>((WIDECHAR*)OutName, Len);
	}
	else
	{
		CopyUnterminatedName((ANSICHAR*)OutName, Len);
		ConvertInPlace<ANSICHAR, TCHAR>((ANSICHAR*)OutName, Len);
	}
}

void FNameEntry::GetAnsiName(ANSICHAR(&Out)[NAME_SIZE]) const
{
	check(!IsWide());
	CopyUnterminatedName(Out, Header.Len);
	Out[Header.Len] = '\0';
}

void FNameEntry::GetWideName(WIDECHAR(&Out)[NAME_SIZE]) const
{
	check(IsWide());
	CopyUnterminatedName(Out, Header.Len);
	Out[Header.Len] = '\0';
}

/** @return null-terminated string */
static const TCHAR* EntryToCString(const FNameEntry& Entry, FNameBuffer& Temp)
{
	if (Entry.IsWide())
	{
		Entry.GetWideName(Temp.WideName);
		return ConvertInPlace<WIDECHAR, TCHAR>(Temp.WideName, Entry.GetNameLength() + 1);
	}
	else
	{
		Entry.GetAnsiName(Temp.AnsiName);
		return ConvertInPlace<ANSICHAR, TCHAR>(Temp.AnsiName, Entry.GetNameLength() + 1);
	}
}

UE::FSharedString FNameEntry::GetPlainNameSharedString() const
{
	checkName(Header.Len != 0);
	FNameBuffer Temp;
	return UE::FSharedString(TStringView<TCHAR>(EntryToCString(*this, Temp), Header.Len));
}

FString FNameEntry::GetPlainNameString() const
{
	checkName(Header.Len != 0);
	FNameBuffer Temp;
	if (Header.bIsWide)
	{
		return FString::ConstructFromPtrSize(GetUnterminatedName(Temp.WideName), Header.Len);
	}
	else
	{
		return FString::ConstructFromPtrSize(GetUnterminatedName(Temp.AnsiName), Header.Len);
	}
}

FUtf8String FNameEntry::GetPlainNameUtf8String() const
{
	checkName(Header.Len != 0);
	FNameBuffer Temp;
	if (Header.bIsWide)
	{
		return FUtf8String::ConstructFromPtrSize(GetUnterminatedName(Temp.WideName), Header.Len);
	}
	else
	{
		return FUtf8String::ConstructFromPtrSize(GetUnterminatedName(Temp.AnsiName), Header.Len);
	}
}

void FNameEntry::AppendNameToString(FString& Out) const
{
	checkName(Header.Len != 0);
	FNameBuffer Temp;
	Out.Append(EntryToCString(*this, Temp), Header.Len);
}

void FNameEntry::AppendNameToString(FUtf8String& Out) const
{
	checkName(Header.Len != 0);
	FNameBuffer Temp;
	Out.Append(EntryToCString(*this, Temp), Header.Len);
}

void FNameEntry::AppendNameToString(FWideStringBuilderBase& Out) const
{
	checkName(Header.Len != 0);
	const int32 Offset = Out.AddUninitialized(Header.Len);
	TCHAR* OutChars = Out.GetData() + Offset;
	if (Header.bIsWide)
	{
		CopyUnterminatedName(reinterpret_cast<WIDECHAR*>(OutChars), Header.Len);
		ConvertInPlace<WIDECHAR, TCHAR>(reinterpret_cast<WIDECHAR*>(OutChars), Header.Len);
	}
	else
	{
		CopyUnterminatedName(reinterpret_cast<ANSICHAR*>(OutChars), Header.Len);
		ConvertInPlace<ANSICHAR, TCHAR>(reinterpret_cast<ANSICHAR*>(OutChars), Header.Len);
	}
}

void FNameEntry::AppendNameToString(FUtf8StringBuilderBase& Out) const
{
	checkName(Header.Len != 0);
	if (Header.bIsWide)
	{
		FNameBuffer DecodeBuffer;
		FNameStringView View = MakeView(DecodeBuffer);
		Out << FWideStringView(View.Wide, View.Len);
	}
	else
	{
		const int32 Offset = Out.AddUninitialized(Header.Len);
		UTF8CHAR* OutChars = Out.GetData() + Offset;
		CopyUnterminatedName(reinterpret_cast<ANSICHAR*>(OutChars), Header.Len);
	}
}

void FNameEntry::AppendAnsiNameToString(FAnsiStringBuilderBase& Out) const
{
	checkName(Header.Len != 0);
	check(!IsWide());
	const int32 Offset = Out.AddUninitialized(Header.Len);
	CopyUnterminatedName(Out.GetData() + Offset, Header.Len);
}

void FNameEntry::AppendNameToPathString(FString& Out) const
{
	checkName(Header.Len != 0);
	FNameBuffer Temp;
	Out.PathAppend(EntryToCString(*this, Temp), Header.Len);
}

#if UE_FNAME_OUTLINE_NUMBER
uint32 FNameEntry::GetNumber() const
{
	checkName(Header.Len == 0);
	return GetNumberedName().Number;
}
#endif

int32 FNameEntry::GetSizeInBytes() const
{
#if UE_FNAME_OUTLINE_NUMBER
	if (Header.Len == 0)
	{
		return FNameEntryAllocator::NumberedEntrySize;
	}
	else
#endif
	{
		return Align(FNameStore::GetSize(*this), alignof(FNameEntry));
	}
}

FNameEntrySerialized::FNameEntrySerialized(const FNameEntry& NameEntry)
{
	bIsWide = NameEntry.IsWide();
	if (bIsWide)
	{
		NameEntry.GetWideName(WideName);
		NonCasePreservingHash = GetRawNonCasePreservingHash(WideName);
		CasePreservingHash = GetRawCasePreservingHash(WideName);
	}
	else
	{
		NameEntry.GetAnsiName(AnsiName);
		NonCasePreservingHash = GetRawNonCasePreservingHash(AnsiName);
		CasePreservingHash = GetRawCasePreservingHash(AnsiName);
	}
}

/**
 * @return FString of name portion minus number.
 */
FString FNameEntrySerialized::GetPlainNameString() const
{
	if (bIsWide)
	{
		return FString(WideName);
	}
	else
	{
		return FString(AnsiName);
	}
}

/*-----------------------------------------------------------------------------
	FName statics.
-----------------------------------------------------------------------------*/

static TArray<FString> NameToDisplayStringExemptions;
static FTransactionallySafeRWLock NameToDisplayStringExemptionLock;

void FName::Reserve(uint32 NumBytes, uint32 NumNames)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("FName Reserve");
	GetNamePoolPostInit().Reserve(NumBytes, NumNames);
}

int64 FName::GetNameEntryMemorySize()
{
	return GetNamePool().NumBlocks() * (int64)FNameEntryAllocator::BlockSizeBytes;
}

int64 FName::GetNameEntryMemoryEstimatedAvailable()
{
	FNamePool& Pool = GetNamePool();
	return (FNameMaxBlocks - Pool.NumBlocks()) * (int64)FNameEntryAllocator::BlockSizeBytes;
}

int64 FName::GetNameTableMemorySize()
{
	return GetNameEntryMemorySize() + sizeof(FNamePool) + GetNamePool().NumSlots() * sizeof(FNameSlot);
}

int32 FName::GetNumAnsiNames()
{
	return GetNamePool().NumAnsiEntries();
}

int32 FName::GetNumWideNames()
{
	return GetNamePool().NumWideEntries();
}

#if UE_FNAME_OUTLINE_NUMBER
int32 FName::GetNumNumberedNames()
{
	return GetNamePool().NumNumberedEntries();
}
#endif

TArray<const FNameEntry*> FName::DebugDump()
{
	return GetNamePool().DebugDump();
}

FNameEntry const* FName::GetEntry(EName Ename)
{
	FNamePool& Pool = GetNamePool();
	return &Pool.Resolve(Pool.Find(Ename));
}

FNameEntry const* FName::GetEntry(FNameEntryId Id)
{
	// Public interface, recurse to the actual string entry if necessary
	return ResolveEntryRecursive(Id);
}

#if UE_TRACE_ENABLED
UE::Trace::FEventRef32 FName::TraceName(const FName& Name)
{
	const FNameEntryId DisplayId = Name.GetDisplayIndexFast();
	return GetNamePool().Trace(DisplayId);
}

void FName::TraceNamesOnConnection()
{
	const FNamePool& Pool = GetNamePool();
	Pool.RetraceAll();
}
#endif // UE_TRACE_ENABLED

FString FName::NameToDisplayString( const FString& InDisplayName, const bool bIsBool )
{
	// Copy the characters out so that we can modify the string in place
	const TArray< TCHAR, FString::AllocatorType >& Chars = InDisplayName.GetCharArray();

	// Characters that can follow the end of a word
	constexpr FAsciiSet EndOfWord(" .,;:?!)");

	UE::TReadScopeLock Lock(NameToDisplayStringExemptionLock);

	const FStringView DisplayNameStringView = InDisplayName;
	const int32 DisplayNameLength = InDisplayName.Len();

	// This is used to indicate that we are in a run of uppercase letter and/or digits.  The code attempts to keep
	// these characters together as breaking them up often looks silly (i.e. "Draw Scale 3 D" as opposed to "Draw Scale 3D"
	bool bInARun = false;
	bool bWasSpace = false;
	bool bWasOpenParen = false;
	bool bWasNumber = false;
	bool bWasMinusSign = false;

	FString OutDisplayName;
	OutDisplayName.GetCharArray().Reserve(Chars.Num());
	for( int32 CharIndex = 0 ; CharIndex < Chars.Num() ; ++CharIndex )
	{
		TCHAR ch = Chars[CharIndex];

		bool bLowerCase = FChar::IsLower( ch );
		bool bUpperCase = FChar::IsUpper( ch );
		bool bIsDigit = FChar::IsDigit( ch );
		bool bIsUnderscore = FChar::IsUnderscore( ch );

		// Skip the first character if the property is a bool (they should all start with a lowercase 'b', which we don't want to keep)
		if( CharIndex == 0 && bIsBool && ch == 'b' )
		{
			// Check if next character is uppercase as it may be a user created string that doesn't follow the rules of Unreal variables
			if (Chars.Num() > 1 && FChar::IsUpper(Chars[1]))
			{
				continue;
			}
		}

		// If this is the start of a word, check if its an exempt word from formatting
		if (CharIndex == 0 || bWasSpace || bWasOpenParen)
		{
			bool bExemptionFound = false;

			for (const FString& Exemption : NameToDisplayStringExemptions)
			{
				if (DisplayNameStringView.Mid(CharIndex).StartsWith(Exemption, ESearchCase::CaseSensitive)) // exempt word
				{
					bExemptionFound = true;

					OutDisplayName += Exemption;

					const int32 ExceptionLength = Exemption.Len();

					if (CharIndex + ExceptionLength < DisplayNameLength && !EndOfWord.Contains(Chars[CharIndex + ExceptionLength]))
					{
						OutDisplayName += TEXT(' ');
						bWasSpace = true;
					}

					CharIndex += ExceptionLength - 1;
					break;
				}
			}

			if (bExemptionFound)
			{
				continue;
			}
		}

		// If the current character is upper case or a digit, and the previous character wasn't, then we need to insert a space if there wasn't one previously
		// We don't do this for numerical expressions, for example "-1.2" should not be formatted as "- 1. 2"
		if( (bUpperCase || (bIsDigit && !bWasMinusSign)) && !bInARun && !bWasOpenParen && !bWasNumber)
		{
			if( !bWasSpace && OutDisplayName.Len() > 0 )
			{
				OutDisplayName += TEXT( ' ' );
				bWasSpace = true;
			}
			bInARun = true;
		}

		// A lower case character will break a run of upper case letters and/or digits
		if( bLowerCase )
		{
			bInARun = false;
		}

		// An underscore denotes a space, so replace it and continue the run
		if( bIsUnderscore )
		{
			ch = TEXT( ' ' );
			bInARun = true;
		}

		// If this is the first character in the string, then it will always be upper-case
		if( OutDisplayName.Len() == 0 )
		{
			ch = FChar::ToUpper( ch );
		}
		else if( !bIsDigit && (bWasSpace || bWasOpenParen))	// If this is first character after a space, then make sure it is case-correct
		{
			// Some words are always forced lowercase
			const TCHAR* Articles[] =
			{
				TEXT( "In" ),
				TEXT( "As" ),
				TEXT( "To" ),
				TEXT( "Or" ),
				TEXT( "At" ),
				TEXT( "On" ),
				TEXT( "If" ),
				TEXT( "Be" ),
				TEXT( "By" ),
				TEXT( "The" ),
				TEXT( "For" ),
				TEXT( "And" ),
				TEXT( "With" ),
				TEXT( "When" ),
				TEXT( "From" ),
			};

			// Search for a word that needs case repaired
			bool bIsArticle = false;
			for( int32 CurArticleIndex = 0; CurArticleIndex < UE_ARRAY_COUNT( Articles ); ++CurArticleIndex )
			{
				// Make sure the character following the string we're testing is not lowercase (we don't want to match "in" with "instance")
				const int32 ArticleLength = FCString::Strlen( Articles[ CurArticleIndex ] );
				if( ( Chars.Num() - CharIndex ) > ArticleLength && !FChar::IsLower( Chars[ CharIndex + ArticleLength ] ) && Chars[ CharIndex + ArticleLength ] != '\0' )
				{
					// Does this match the current article?
					if( FCString::Strncmp( &Chars[ CharIndex ], Articles[ CurArticleIndex ], ArticleLength ) == 0 )
					{
						bIsArticle = true;
						break;
					}
				}
			}

			// Start of a keyword, force to lowercase
			if( bIsArticle )
			{
				ch = FChar::ToLower( ch );				
			}
			else	// First character after a space that's not a reserved keyword, make sure it's uppercase
			{
				ch = FChar::ToUpper( ch );
			}
		}

		bWasSpace = ( ch == TEXT( ' ' ) ? true : false );
		bWasOpenParen = ( ch == TEXT( '(' ) ? true : false );

		// What could be included as part of a numerical representation.
		// For example -1.2
		bWasMinusSign = (ch == TEXT('-'));
		const bool bPotentialNumericalChar = bWasMinusSign || (ch == TEXT('.'));
		bWasNumber = bIsDigit || (bWasNumber && bPotentialNumericalChar);

		OutDisplayName += ch;
	}

	return OutDisplayName;
}

void FName::AddNameToDisplayStringExemption(const FString& InExemption)
{
	UE::TWriteScopeLock Lock(NameToDisplayStringExemptionLock);

	NameToDisplayStringExemptions.Add(InExemption);
}

void FName::RemoveNameToDisplayStringExemption(const FString& InExemption)
{
	UE::TWriteScopeLock Lock(NameToDisplayStringExemptionLock);

	NameToDisplayStringExemptions.Remove(InExemption);
}

const EName* FName::ToEName() const
{
	return GetNamePoolPostInit().FindEName(ComparisonIndex);
}

bool FName::IsWithinBounds(FNameEntryId Id)
{
	return GetNamePoolPostInit().IsValid(Id);
}

#if DO_CHECK
void FName::ReportInvalidInternalNumber(FNameEntryId ComparisonId, int32 InternalNumber)
{
	UE_LOGF(LogCore, Fatal, "Trying to construct an invalid FName '%ls' with internal number: %d", *GetNamePool().Resolve(ComparisonId).GetPlainNameString(), InternalNumber);
	for (;;);
}
#endif

/*-----------------------------------------------------------------------------
	FName implementation.
-----------------------------------------------------------------------------*/

struct FNameAnsiStringView
{
	using CharType = ANSICHAR;

	const ANSICHAR* Str;
	int32 Len;
};

struct FNameUtf8StringViewWithWidth
{
	using CharType = UTF8CHAR;

	const UTF8CHAR* Str;
	int32 Len;
	bool bIsWide;
};

struct FWideStringViewWithWidth
{
	using CharType = WIDECHAR;

	const WIDECHAR* Str;
	int32 Len;
	bool bIsWide;
};

static FNameAnsiStringView MakeUnconvertedView(const ANSICHAR* Str, int32 Len)
{
	return { Str, Len };
}

static FNameAnsiStringView MakeUnconvertedView(const ANSICHAR* Str)
{
	return { Str, Str ? FCStringAnsi::Strlen(Str) : 0 };
}

template <typename CharType>
static bool IsWide(const CharType* Str, const int32 Len)
{
	uint32 UserCharBits = 0;
	for (int32 I = 0; I < Len; ++I)
	{
		UserCharBits |= TChar<CharType>::ToUnsigned(Str[I]);
	}
	return UserCharBits & 0xffffff80u;
}

template <typename CharType>
static int32 GetLengthAndWidth(const CharType* Str, bool& bOutIsWide)
{
	uint32 UserCharBits = 0;
	const CharType* It = Str;
	if (Str)
	{
		while (*It)
		{
			UserCharBits |= TChar<CharType>::ToUnsigned(*It);
			++It;
		}
	}

	bOutIsWide = UserCharBits & 0xffffff80u;

	return UE_PTRDIFF_TO_INT32(It - Str);
}

static FNameUtf8StringViewWithWidth MakeUnconvertedView(const UTF8CHAR* Str, int32 Len)
{
	return { Str, Len, IsWide(Str, Len) };
}

static FNameUtf8StringViewWithWidth MakeUnconvertedView(const UTF8CHAR* Str)
{
	FNameUtf8StringViewWithWidth View;
	View.Str = Str;
	View.Len = GetLengthAndWidth(Str, View.bIsWide);
	return View;
}

static FWideStringViewWithWidth MakeUnconvertedView(const WIDECHAR* Str, int32 Len)
{
	return { Str, Len, IsWide(Str, Len) };
}

static FWideStringViewWithWidth MakeUnconvertedView(const WIDECHAR* Str)
{
	FWideStringViewWithWidth View;
	View.Str = Str;
	View.Len = GetLengthAndWidth(Str, View.bIsWide);
	return View;
}

/** Templated implementations of non-templated member functions, helps keep header clean */
struct FNameHelper
{
	template<typename ViewType>
	static FName MakeDetectNumber(ViewType View, EFindName FindType)
	{
		if (View.Len == 0)
		{
			return FName();
		}
		
		uint32 InternalNumber = UE::Core::Private::ParseNumberFromName(View.Str, /* may be shortened */ View.Len);
		return MakeWithInternalNumber(View, FindType, InternalNumber);
	}

	static FName MakeWithInternalNumber(FNameAnsiStringView	 View, EFindName FindType, int32 InternalNumber)
	{
		// Ignore the supplied number if the name string is empty
		// to keep the semantics of the old FName implementation
		if (View.Len == 0)
		{
			return FName();
		}

		return MakeInternal(FNameStringView(View.Str, View.Len), FindType, InternalNumber);
	}

	static FName MakeWithInternalNumber(FNameUtf8StringViewWithWidth View, EFindName FindType, int32 InternalNumber)
	{
		// Ignore the supplied number if the name string is empty
		// to keep the semantics of the old FName implementation
		if (View.Len == 0)
		{
			return FName();
		}

		if (!View.bIsWide)
		{
			return MakeInternal(FNameStringView(reinterpret_cast<const ANSICHAR*>(View.Str), View.Len), FindType, InternalNumber);
		}
		else
		{
			TStringConversion<FUTF8ToTCHAR_Convert, NAME_SIZE> WideName(View.Str, View.Len);
			return MakeInternal(FNameStringView(WideName.Get(), WideName.Length()), FindType, InternalNumber);
		}
	}

	static FName MakeWithInternalNumber(const FWideStringViewWithWidth View, EFindName FindType, int32 InternalNumber)
	{
		// Ignore the supplied number if the name string is empty
		// to keep the semantics of the old FName implementation
		if (View.Len == 0)
		{
			return FName();
		}

		// Convert to narrow if possible
		if (!View.bIsWide)
		{
			// Consider _mm_packus_epi16 or similar if this proves too slow
			ANSICHAR AnsiName[NAME_SIZE];
			for (int32 I = 0, Len = FMath::Min<int32>(View.Len, NAME_SIZE); I < Len; ++I)
			{
				AnsiName[I] = View.Str[I];
			}
			return MakeInternal(FNameStringView(AnsiName, View.Len), FindType, InternalNumber);
		}
		else
		{
			return MakeInternal(FNameStringView(View.Str, View.Len), FindType, InternalNumber);
		}
	}

	static FName MakeFromLoaded(const FNameEntrySerialized& LoadedEntry)
	{
		FNameStringView View = LoadedEntry.bIsWide
			? FNameStringView(LoadedEntry.WideName, FCStringWide::Strlen(LoadedEntry.WideName))
			: FNameStringView(LoadedEntry.AnsiName, FCStringAnsi::Strlen(LoadedEntry.AnsiName));

		return MakeInternal(View, FNAME_Add, NAME_NO_NUMBER_INTERNAL);
	}

	static FName MakeWithInternalNumber(FNameEntryIds BaseIds, EFindName FindType, int32 InternalNumber)
	{
#if UE_FNAME_OUTLINE_NUMBER
#if WITH_CASE_PRESERVING_NAME
		// Advanced users must pass in matching display & comparison ids
		checkName(FName::ResolveEntry(BaseIds.DisplayId)->ComparisonId == BaseIds.ComparisonId);
#endif

		// If BaseIds are NAME_None, we want to produce a numbered suffix of None. 
		// If anything searches for BaseIds they need to validate whether they want to pass NAME_None on or not before calling this function.
		if (InternalNumber == NAME_NO_NUMBER_INTERNAL)
		{
			return FinalConstruct(BaseIds);
		}

		FNamePool& Pool = GetNamePool();
		if (FindType == FNAME_Add)
		{
			FNameEntryId DisplayId;
			UE_AUTORTFM_OPEN
			{
				DisplayId = Pool.StoreWithNumber(BaseIds, InternalNumber);
			};
			return FinalConstruct(FNameEntryIds{ ResolveComparisonId(DisplayId), DisplayId });
		}
		else
		{
			check(FindType == FNAME_Find);
			FNameEntryId DisplayId = Pool.FindWithNumber(BaseIds.DisplayId, InternalNumber);
			if (DisplayId)
			{
				return FinalConstruct(FNameEntryIds{ ResolveComparisonId(DisplayId), DisplayId });
			}
			else
			{
				// Not found
				return FName();
			}
		}
#else
		// Number is just stored in the FName pass it straight on
		return FinalConstruct(BaseIds, InternalNumber);
#endif
	}

	template<typename CharType>
	static bool EqualsString(FName Name, const CharType* Str, int32 Len)
	{
		check(Len >= 0);

		// Make NAME_None == TEXTVIEW("") or TEXTVIEW(nullptr) consistent with NAME_None == FName(TEXT("")) or FName(nullptr)
		if (Len == 0)
		{
			return Name.IsNone();
		}

		// "_[num]" is considered "None" to keep semantics of old FName implementation
		if (Name.IsNone() & (Str[0] == '_'))
		{
			UE::Core::Private::ParseNumberFromName(Str, /* in-out */ Len);
			return Len == 0;
		}

		const FNameEntry* Entry = FName::ResolveEntry(Name.GetComparisonIndexInternal());

	#if UE_FNAME_OUTLINE_NUMBER
		int32 Number = Entry->IsNumbered() ? Entry->GetNumber() : NAME_NO_NUMBER_INTERNAL;
		Entry = Entry->IsNumbered() ? FName::ResolveEntry(Entry->GetNumberedName().Id) : Entry; 
		check(!Entry->IsNumbered());
	#else
		int32 Number = Name.GetNumber();
	#endif

		// Check _[number] suffix first and shorten Len if it matches
		if (Number != NAME_NO_NUMBER_INTERNAL && 
			Number != UE::Core::Private::ParseNumberFromName(Str, /* in-out */ Len)) 
		{
			return false;
		}

		if (Len != Entry->GetNameLength())
		{
			return false;
		}
		
		FNameBuffer Temp;
		return Entry->IsWide()
			? !FPlatformString::Strnicmp(Str, Entry->GetUnterminatedName(Temp.WideName), Len)
			: !FPlatformString::Strnicmp(Str, Entry->GetUnterminatedName(Temp.AnsiName), Len);
	}

	// Internal helpers for Make* functions
private:

	static FName MakeInternal(FNameStringView View, EFindName FindType, int32 InternalNumber)
	{
		FNameEntryIds Ids;
		UE_AUTORTFM_OPEN
		{
			Ids = FindOrStoreString(View, FindType);
		};
#if UE_FNAME_OUTLINE_NUMBER
		if (FindType == FNAME_Find && !Ids.DisplayId)
		{
			// We need to disambiguate here between "we found 'None'" and "we didn't find what we were looking for"
			if (View.IsNoneString())
			{
				return MakeWithInternalNumber(Ids, FindType, InternalNumber); // 'None' with suffix
			}
			else
			{
				return FName(); // Not found
			}
		}
#endif

#if DO_CHECK
		if (!FName::IsValidInternalNumber(InternalNumber))
		{
			FName::ReportInvalidInternalNumber(Ids.ComparisonId, InternalNumber);
		}
#endif

		return MakeWithInternalNumber(Ids, FindType, InternalNumber);
	}

	// Indices have already been numbered if necessary
	// Not an FName constructor because of implementation details around UE_FNAME_OUTLINE_NUMBER that we want to hide from FName interface
#if UE_FNAME_OUTLINE_NUMBER
	static FName FinalConstruct(FNameEntryIds Ids)
#else
	static FName FinalConstruct(FNameEntryIds Ids, int32 InternalNumber)
#endif // UE_FNAME_OUTLINE_NUMBER
	{
		FName Out;
		Out.ComparisonIndex = Ids.ComparisonId;
#if WITH_CASE_PRESERVING_NAME
		Out.DisplayIndex = Ids.DisplayId;
#endif
#if !UE_FNAME_OUTLINE_NUMBER
		Out.Number = InternalNumber;
#endif
		return Out;
	}

#if WITH_CASE_PRESERVING_NAME
	static FNameEntryId ResolveComparisonId(FNameEntryId DisplayId)
	{
		if (DisplayId.IsNone()) { return FNameEntryId(); }

		return GetNamePool().Resolve(DisplayId).ComparisonId;

	}
#else
	static FNameEntryId ResolveComparisonId(FNameEntryId DisplayId)
	{
		return DisplayId;
	}
#endif

	// Find or store a plain string name entry, returning both comparison and display id for it.
	// If not found, returns NAME_None for both indices.
	static FNameEntryIds FindOrStoreString(FNameStringView View, EFindName FindType)
	{
		FNameEntryIds Result{};

		UE_AUTORTFM_OPEN
		{
			if (View.Len >= NAME_SIZE)
			{
				// If we're doing a find, and the string is too long, then clearly we didn't find it
				if (FindType == FNAME_Find)
				{
					Result = FNameEntryIds{};
				}
				else
				{
					checkf(false, TEXT("FName's %d max length exceeded. Got %d characters excluding null-terminator:\n%.*s"), 
						NAME_SIZE - 1, View.Len, NAME_SIZE, View.IsAnsi() ? ANSI_TO_TCHAR(View.Ansi) : View.Wide)

					const ANSICHAR* ErrorString = "ERROR_NAME_SIZE_EXCEEDED";
					Result = FindOrStoreString(FNameStringView(ErrorString, FCStringAnsi::Strlen(ErrorString), false), FNAME_Add);
				}
			}
			else
			{
				FNamePool& Pool = GetNamePool();

				if (FindType == FNAME_Add)
				{
					FNameEntryId DisplayId = Pool.Store(View);
					Result = FNameEntryIds{ ResolveComparisonId(DisplayId), DisplayId };
				}
				else
				{
					check(FindType == FNAME_Find);
					FNameEntryId DisplayId = Pool.Find(View);
					Result = FNameEntryIds{ ResolveComparisonId(DisplayId), DisplayId };
				}
			}
		};

		return Result;
	}

#if UE_FNAME_OUTLINE_NUMBER
	static FName FindNumbered(FNameEntryId InDisplayIndex, int32 InternalNumber)
	{
		FNamePool& Pool = GetNamePool();
		FNameEntryId DisplayId = Pool.FindWithNumber(InDisplayIndex, InternalNumber);
		if (DisplayId)
		{
			FNameEntryId ComparisonId = ResolveComparisonId(DisplayId);
			return FinalConstruct(FNameEntryIds{ ComparisonId, DisplayId });
		}
		else
		{
			return FName();
		}
	}
#endif // UE_FNAME_OUTLINE_NUMBER
};


#if WITH_CASE_PRESERVING_NAME
FNameEntryId FName::GetComparisonIdFromDisplayId(FNameEntryId DisplayId)
{
	return GetEntry(DisplayId)->ComparisonId;
}
#endif

#if UE_FNAME_OUTLINE_NUMBER
FNameEntryId FName::GetComparisonIndex() const
{
	const FNameEntry& Entry = GetNamePool().Resolve(ComparisonIndex);
	if (Entry.Header.Len == 0)
	{
		return Entry.GetNumberedName().Id;
	}
	else
	{
		return ComparisonIndex;
	}
}

FNameEntryId FName::GetDisplayIndex() const
{
#if WITH_CASE_PRESERVING_NAME
	const FNameEntry& Entry = GetNamePool().Resolve(DisplayIndex);
	if (Entry.Header.Len == 0)
	{
		return Entry.GetNumberedName().Id;
	}
	else
	{
		return DisplayIndex;
	}
#else
	return GetComparisonIndex();
#endif
}

FName FName::FindNumberedName(FNameEntryId DisplayId, int32 InInternalNumber)
{
	return FNameHelper::MakeWithInternalNumber(FNameEntryIds{FName::GetComparisonIdFromDisplayId(DisplayId), DisplayId}, FNAME_Find, InInternalNumber);
}

int32 FName::GetNumber() const 
{
	FNameEntryId Id = GetDisplayIndexInternal();

	const FNameEntry* Entry = ResolveEntry(Id);
	if (Entry->Header.Len == 0)
	{
		return Entry->GetNumberedName().Number;
	}
	else
	{
		return NAME_NO_NUMBER_INTERNAL;
	}
}
void FName::SetNumber(int32 InInternalNumber) 
{
	FName NewName = FNameHelper::MakeWithInternalNumber(FNameEntryIds{GetComparisonIndex(), GetDisplayIndex()}, FNAME_Add, InInternalNumber);
	*this = NewName;
}
#endif // UE_FNAME_OUTLINE_NUMBER

// Resolve the entry directly referred to by LookupId
const FNameEntry* FName::ResolveEntry(FNameEntryId LookupId)
{
	return &GetNamePool().Resolve(LookupId);
}

// Recursively resolve through the entry referred to by LookupId to reach the allocated string entry, in the case of UE_FNAME_OUTLINE_NUMBER=1
const FNameEntry* FName::ResolveEntryRecursive(FNameEntryId LookupId)
{
	const FNameEntry* Entry = ResolveEntry(LookupId);
#if UE_FNAME_OUTLINE_NUMBER
	if (Entry->Header.Len == 0)
	{
		return ResolveEntry(Entry->GetNumberedName().Id); // Should only ever recurse one level
	}
	else
#endif
	{
		return Entry;
	}
}


FName::FName(const WIDECHAR* Name)
	: FName(FNameHelper::MakeDetectNumber(MakeUnconvertedView(Name), FNAME_Add))
{}

FName::FName(const ANSICHAR* Name)
	: FName(FNameHelper::MakeDetectNumber(MakeUnconvertedView(Name), FNAME_Add))
{}

FName::FName(const UTF8CHAR* Name)
	: FName(FNameHelper::MakeDetectNumber(MakeUnconvertedView(Name), FNAME_Add))
{}

FName::FName(const WIDECHAR* Name, EFindName FindType)
	: FName(FNameHelper::MakeDetectNumber(MakeUnconvertedView(Name), FindType))
{}

FName::FName(const ANSICHAR* Name, EFindName FindType)
	: FName(FNameHelper::MakeDetectNumber(MakeUnconvertedView(Name), FindType))
{}

FName::FName(const UTF8CHAR* Name, EFindName FindType)
	: FName(FNameHelper::MakeDetectNumber(MakeUnconvertedView(Name), FindType))
{}

FName::FName(int32 Len, const WIDECHAR* Name, EFindName FindType)
	: FName(FNameHelper::MakeDetectNumber(MakeUnconvertedView(Name, Len), FindType))
{}

FName::FName(int32 Len, const ANSICHAR* Name, EFindName FindType)
	: FName(FNameHelper::MakeDetectNumber(MakeUnconvertedView(Name, Len), FindType))
{}

FName::FName(int32 Len, const UTF8CHAR* Name, EFindName FindType)
	: FName(FNameHelper::MakeDetectNumber(MakeUnconvertedView(Name, Len), FindType))
{}

FName::FName(const WIDECHAR* Name, int32 InInternalNumber)
	: FName(FNameHelper::MakeWithInternalNumber(MakeUnconvertedView(Name), FNAME_Add, InInternalNumber))
{}

FName::FName(const ANSICHAR* Name, int32 InInternalNumber)
	: FName(FNameHelper::MakeWithInternalNumber(MakeUnconvertedView(Name), FNAME_Add, InInternalNumber))
{}

FName::FName(const UTF8CHAR* Name, int32 InInternalNumber)
	: FName(FNameHelper::MakeWithInternalNumber(MakeUnconvertedView(Name), FNAME_Add, InInternalNumber))
{}

FName::FName(int32 Len, const WIDECHAR* Name, int32 InInternalNumber)
	: FName(InInternalNumber != NAME_NO_NUMBER_INTERNAL ? FNameHelper::MakeWithInternalNumber(MakeUnconvertedView(Name, Len), FNAME_Add, InInternalNumber)
		: FNameHelper::MakeDetectNumber(MakeUnconvertedView(Name, Len), FNAME_Add))
{}

FName::FName(int32 Len, const ANSICHAR* Name, int32 InInternalNumber)
	: FName(InInternalNumber != NAME_NO_NUMBER_INTERNAL ? FNameHelper::MakeWithInternalNumber(MakeUnconvertedView(Name, Len), FNAME_Add, InInternalNumber)
		: FNameHelper::MakeDetectNumber(MakeUnconvertedView(Name, Len), FNAME_Add))
{}

FName::FName(int32 Len, const UTF8CHAR* Name, int32 InInternalNumber)
	: FName(InInternalNumber != NAME_NO_NUMBER_INTERNAL ? FNameHelper::MakeWithInternalNumber(MakeUnconvertedView(Name, Len), FNAME_Add, InInternalNumber)
		: FNameHelper::MakeDetectNumber(MakeUnconvertedView(Name, Len), FNAME_Add))
{}

FName::FName(const TCHAR* Name, int32 InInternalNumber, bool bSplitName)
	: FName(InInternalNumber == NAME_NO_NUMBER_INTERNAL && bSplitName
		? FNameHelper::MakeDetectNumber(MakeUnconvertedView(Name), FNAME_Add)
		: FNameHelper::MakeWithInternalNumber(MakeUnconvertedView(Name), FNAME_Add, InInternalNumber))
{}

FName::FName(const FNameEntrySerialized& LoadedEntry)
	: FName(FNameHelper::MakeFromLoaded(LoadedEntry))
{}

bool FName::Equals(FName Name, FAnsiStringView View)
{
	return FNameHelper::EqualsString(Name, View.GetData(), View.Len());
}

bool FName::Equals(FName Name, FWideStringView View)
{
	return FNameHelper::EqualsString(Name, View.GetData(), View.Len());
}

bool FName::Equals(FName Name, const ANSICHAR* Str)
{
	return Equals(Name, FAnsiStringView(Str));
}

bool FName::Equals(FName Name, const WIDECHAR* Str)
{
	return Equals(Name, FWideStringView(Str));
}

template <ESearchCase::Type SearchCase>
int32 FName::CompareInternal( const FName& Other ) const
{
#if UE_FNAME_OUTLINE_NUMBER
	if constexpr (SearchCase == ESearchCase::IgnoreCase)
	{
		if (GetComparisonIndexInternal() == Other.GetComparisonIndexInternal())
		{
			return 0; // Identical
		}
	}
	else
	{
		if (GetDisplayIndexInternal() == Other.GetDisplayIndexInternal())
		{
			return 0; // Identical
		}
	}
#endif // UE_FNAME_OUTLINE_NUMBER

	FNameEntryId A, B;
	if constexpr (SearchCase == ESearchCase::IgnoreCase)
	{
		A = GetComparisonIndex();
		B = Other.GetComparisonIndex();
	}
	else
	{
		A = GetDisplayIndexFast();
		B = Other.GetDisplayIndexFast();
	}

	// Names match, check whether numbers match.
	if (A == B)
	{
		return GetNumber() - Other.GetNumber();
	}

	// Names don't match. This means we don't even need to check numbers.
	return CompareDifferentIdsAlphabetically<SearchCase>(A, B);
}

int32 FName::Compare( const FName& Other ) const
{
	return CompareInternal<ESearchCase::IgnoreCase>(Other);
}

int32 FName::CompareSensitive( const FName& Other ) const
{
	return CompareInternal<ESearchCase::CaseSensitive>(Other);
}

uint32 FName::GetPlainNameString(TCHAR(&OutName)[NAME_SIZE]) const
{
	const FNameEntry& Entry = *GetDisplayNameEntry();
	Entry.GetName(OutName);
	return Entry.GetNameLength();
}

FString FName::GetPlainNameString() const
{
	return GetDisplayNameEntry()->GetPlainNameString();
}

UE::FSharedString FName::GetPlainNameSharedString() const
{
	return GetDisplayNameEntry()->GetPlainNameSharedString();
}

void FName::GetPlainANSIString(ANSICHAR(&AnsiName)[NAME_SIZE]) const
{
	GetDisplayNameEntry()->GetAnsiName(AnsiName);
}

void FName::GetPlainWIDEString(WIDECHAR(&WideName)[NAME_SIZE]) const
{
	GetDisplayNameEntry()->GetWideName(WideName);
}

const FNameEntry* FName::GetComparisonNameEntry() const
{
	return ResolveEntryRecursive(GetComparisonIndexInternal());
}

const FNameEntry* FName::GetDisplayNameEntry() const
{
	return ResolveEntryRecursive(GetDisplayIndexInternal());
}

FString FName::ToString() const
{
// With UE_FNAME_OUTLINE_NUMBER reading number isn't free so skip this check
#if !UE_FNAME_OUTLINE_NUMBER
	if (GetNumber() == NAME_NO_NUMBER_INTERNAL)
	{
		// Avoids some extra allocations in non-number case
		return GetDisplayNameEntry()->GetPlainNameString();
	}
#endif // !UE_FNAME_OUTLINE_NUMBER
	
	FString Out;
	ToString(Out);
	return Out;
}

FUtf8String FName::ToUtf8String() const
{
// With UE_FNAME_OUTLINE_NUMBER reading number isn't free so skip this check
#if !UE_FNAME_OUTLINE_NUMBER
	if (GetNumber() == NAME_NO_NUMBER_INTERNAL)
	{
		// Avoids some extra allocations in non-number case
		return GetDisplayNameEntry()->GetPlainNameUtf8String();
	}
#endif // !UE_FNAME_OUTLINE_NUMBER
	
	FUtf8String Out;
	ToString(Out);
	return Out;
}

FString LexToString(const FName& Name)
{
	return Name.ToString();
}

void FName::ToString(FWideString& Out) const
{
#if UE_FNAME_OUTLINE_NUMBER
	FNameEntryId Id = GetDisplayIndexInternal();
	const FNameEntry* ThisNameEntry = ResolveEntry(Id);
	if (ThisNameEntry->Header.Len != 0)
	{
		// No number
		Out.Reset(ThisNameEntry->GetNameLength());
		ThisNameEntry->AppendNameToString(Out);
	}
	else
	{
		const FNameEntry* BaseEntry = ResolveEntry(ThisNameEntry->GetNumberedName().Id);
		Out.Reset(BaseEntry->GetNameLength() + 6);
		BaseEntry->AppendNameToString(Out);

		Out += TEXT('_');
		Out.AppendInt(NAME_INTERNAL_TO_EXTERNAL(ThisNameEntry->GetNumber()));
	}
#else // UE_FNAME_OUTLINE_NUMBER
	// A version of ToString that saves at least one string copy
	const FNameEntry* const NameEntry = GetDisplayNameEntry();

	if (GetNumber() == NAME_NO_NUMBER_INTERNAL)
	{
		Out.Reset(NameEntry->GetNameLength());
		NameEntry->AppendNameToString(Out);
	}
	else
	{
		Out.Reset(NameEntry->GetNameLength() + 6);
		NameEntry->AppendNameToString(Out);

		Out += TEXT('_');
		Out.AppendInt(NAME_INTERNAL_TO_EXTERNAL(GetNumber()));
	}
#endif // UE_FNAME_OUTLINE_NUMBER
}

void FName::ToString(FUtf8String& Out) const
{
#if UE_FNAME_OUTLINE_NUMBER
	FNameEntryId Id = GetDisplayIndexInternal();
	const FNameEntry* ThisNameEntry = ResolveEntry(Id);
	if (ThisNameEntry->Header.Len != 0)
	{
		// No number
		Out.Reset(ThisNameEntry->GetNameLengthUtf8());
		ThisNameEntry->AppendNameToString(Out);
	}
	else
	{
		const FNameEntry* BaseEntry = ResolveEntry(ThisNameEntry->GetNumberedName().Id);
		Out.Reset(BaseEntry->GetNameLengthUtf8() + 6);
		BaseEntry->AppendNameToString(Out);

		Out += '_';
		Out.AppendInt(NAME_INTERNAL_TO_EXTERNAL(ThisNameEntry->GetNumber()));
	}
#else // UE_FNAME_OUTLINE_NUMBER
	// A version of ToString that saves at least one string copy
	const FNameEntry* const NameEntry = GetDisplayNameEntry();

	if (GetNumber() == NAME_NO_NUMBER_INTERNAL)
	{
		Out.Reset(NameEntry->GetNameLengthUtf8());
		NameEntry->AppendNameToString(Out);
	}
	else
	{
		Out.Reset(NameEntry->GetNameLengthUtf8() + 6);
		NameEntry->AppendNameToString(Out);

		Out += '_';
		Out.AppendInt(NAME_INTERNAL_TO_EXTERNAL(GetNumber()));
	}
#endif // UE_FNAME_OUTLINE_NUMBER
}

void FName::ToString(FWideStringBuilderBase& Out) const
{
	Out.Reset();
	AppendString(Out);
}

void FName::ToString(FUtf8StringBuilderBase& Out) const
{
	Out.Reset();
	AppendString(Out);
}

uint32 FName::GetStringLength() const
{
	const FNameEntry& Entry = *GetDisplayNameEntry();
	uint32 NameLen = Entry.GetNameLength();

	if (GetNumber() != NAME_NO_NUMBER_INTERNAL)
	{
		// Return the length of the number suffix "_<num>"
		// We only need to actually compute the size past values of 9.
		// The minimum suffix length is 2, including the one for a suffix value of 0.
		NameLen += 2;
		for (int32 Num = NAME_INTERNAL_TO_EXTERNAL(GetNumber()); Num >= 10; Num /= 10)
		{
			++NameLen;
		}
	}
	return NameLen;
}

uint32 FName::ToStringInternal(TCHAR* Out, const uint32 OutSize) const
{
	check(OutSize > 0);
	const uint32 OutLen = OutSize - 1;

	const FNameEntry& Entry = *GetDisplayNameEntry();
	const uint32 NameLen = Entry.GetNameLength();
	Entry.GetUnterminatedName(Out, OutLen);

	if (GetNumber() != NAME_NO_NUMBER_INTERNAL && OutLen > NameLen)
	{
		int32 SuffixLen = FCString::Snprintf(Out + NameLen, OutSize - NameLen, TEXT("_%d"), NAME_INTERNAL_TO_EXTERNAL(GetNumber()));
		if (SuffixLen < 0)
		{
			// Suffix was truncated
			return OutLen;
		}
		else
		{
			// Suffix was fully written
			return NameLen + SuffixLen;
		}
	}
	else
	{
		// No suffix, or no room to add it
		uint32 NullIndex = FMath::Min<uint32>(OutLen, NameLen);
		Out[NullIndex] = TEXT('\0');
		return NullIndex;
	}
}

void FName::AppendString(FWideString& Out) const
{
	const FNameEntry* const NameEntry = GetDisplayNameEntry();

	NameEntry->AppendNameToString( Out );
	if (GetNumber() != NAME_NO_NUMBER_INTERNAL)
	{
		Out += TEXT('_');
		Out.AppendInt(NAME_INTERNAL_TO_EXTERNAL(GetNumber()));
	}
}

void FName::AppendString(FUtf8String& Out) const
{
	const FNameEntry* const NameEntry = GetDisplayNameEntry();

	NameEntry->AppendNameToString( Out );
	if (GetNumber() != NAME_NO_NUMBER_INTERNAL)
	{
		Out += '_';
		Out.AppendInt(NAME_INTERNAL_TO_EXTERNAL(GetNumber()));
	}
}

template <typename StringBuilderType>
void FName::AppendStringInternal(StringBuilderType& Out) const
{
#if UE_FNAME_OUTLINE_NUMBER
	FNameEntryId Id = GetDisplayIndexInternal();
	const FNameEntry* ThisNameEntry = ResolveEntry(GetDisplayIndexInternal());

	if (ThisNameEntry->Header.Len != 0)
	{
		// No number
		ThisNameEntry->AppendNameToString(Out);
	}
	else
	{
		const FNameEntry* BaseEntry = ResolveEntry(ThisNameEntry->GetNumberedName().Id);
		BaseEntry->AppendNameToString(Out);
		Out << '_' << NAME_INTERNAL_TO_EXTERNAL(ThisNameEntry->GetNumber());
	}

#else // UE_FNAME_OUTLINE_NUMBER
	GetDisplayNameEntry()->AppendNameToString(Out);

	const int32 InternalNumber = GetNumber();
	if (InternalNumber != NAME_NO_NUMBER_INTERNAL)
	{
		Out << '_' << NAME_INTERNAL_TO_EXTERNAL(InternalNumber);
	}
#endif // UE_FNAME_OUTLINE_NUMBER
}

void FName::AppendString(FWideStringBuilderBase& Out) const
{
	AppendStringInternal(Out);
}

void FName::AppendString(FUtf8StringBuilderBase& Out) const
{
	AppendStringInternal(Out);
}

bool FName::TryAppendAnsiString(FAnsiStringBuilderBase& Out) const
{
	const FNameEntry* const NameEntry = GetDisplayNameEntry();

	if (NameEntry->IsWide())
	{
		return false;
	}

	NameEntry->AppendAnsiNameToString(Out);

	const int32 InternalNumber = GetNumber();
	if (InternalNumber != NAME_NO_NUMBER_INTERNAL)
	{
		Out << '_' << NAME_INTERNAL_TO_EXTERNAL(InternalNumber);
	}

	return true;
}

void AppendHash(FBlake3& Builder, FName In)
{
	FNameBuffer DecodeBuffer;
	FNameStringView NameEntryView = In.GetDisplayNameEntry()->MakeView(DecodeBuffer);
	if (NameEntryView.IsAnsi())
	{
		Builder.Update(NameEntryView.Ansi, NameEntryView.Len * sizeof(NameEntryView.Ansi[0]));
	}
	else
	{
		Builder.Update(NameEntryView.Wide, NameEntryView.Len * sizeof(NameEntryView.Wide[0]));
	}
	int32 Number = INTEL_ORDER64(In.GetNumber());
	Builder.Update(&Number, sizeof(Number));
}

void FName::DisplayHash(FOutputDevice& Ar)
{
	GetNamePool().LogStats(Ar);
}

FString FName::SafeString(FNameEntryId InDisplayIndex, int32 InstanceNumber)
{
#if UE_FNAME_OUTLINE_NUMBER
	const FNameEntry* Entry = &GetNamePool().Resolve(InDisplayIndex);
	if (!Entry)
	{
		return FString();
	}

	if (InstanceNumber == NAME_NO_NUMBER_INTERNAL)
	{
		// Avoids some extra allocations in non-number case
		return Entry->GetPlainNameString();
	}

	TStringBuilder<FName::StringBufferSize> Builder;
	Entry->AppendNameToString(Builder);
	Builder << TEXT("_") << NAME_INTERNAL_TO_EXTERNAL(InstanceNumber);
	return FString(Builder);
#else // UE_FNAME_OUTLINE_NUMBER
	return FName(InDisplayIndex, InDisplayIndex, InstanceNumber).ToString();
#endif // UE_FNAME_OUTLINE_NUMBER
}

bool FName::IsValidXName() const
{
	return IsValidXName(*this, FString(INVALID_NAME_CHARACTERS), (FText*)nullptr, (const FText*)nullptr);
}

bool FName::IsValidXName(FText& OutReason) const
{
	return IsValidXName(*this, FString(INVALID_NAME_CHARACTERS), &OutReason);
}

bool FName::IsValidXName(const FName InName, const FString& InInvalidChars, FText* OutReason, const FText* InErrorCtx)
{
	return IsValidXName(FNameBuilder(InName), InInvalidChars, OutReason, InErrorCtx);
}

bool FName::IsValidXName(const TCHAR* InName, const FString& InInvalidChars, FText* OutReason, const FText* InErrorCtx)
{
	return IsValidXName(FStringView(InName), InInvalidChars, OutReason, InErrorCtx);
}

bool FName::IsValidXName(const FString& InName, const FString& InInvalidChars, FText* OutReason, const FText* InErrorCtx)
{
	return IsValidXName(FStringView(InName), InInvalidChars, OutReason, InErrorCtx);
}

bool FName::IsValidXName(const FStringView& InName, const FString& InInvalidChars, FText* OutReason, const FText* InErrorCtx)
{
	if (InName.IsEmpty() || InInvalidChars.IsEmpty())
	{
		return true;
	}

	enum class EInvalidCharacters
	{
		Space = 32,
		LineFeed = 10,
		CarriageReturn = 13,
		Tab = 9
	};
	
	// See if the name contains invalid characters.
	FString MatchedInvalidChars;
	TArray<FText> MatchedInvalidWhitespaceChars;
	TSet<TCHAR> AlreadyMatchedInvalidChars;
		
	for (const TCHAR InvalidChar : InInvalidChars)
	{
		int32 InvalidCharIndex = INDEX_NONE;
		if (!AlreadyMatchedInvalidChars.Contains(InvalidChar) && InName.FindChar(InvalidChar, InvalidCharIndex))
		{
			switch (static_cast<EInvalidCharacters>(InvalidChar))
			{
			case EInvalidCharacters::Space:
				MatchedInvalidWhitespaceChars.Add(NSLOCTEXT("Core", "FNameSpaceName", "space"));
				break;
			case EInvalidCharacters::CarriageReturn:
				MatchedInvalidWhitespaceChars.Add(NSLOCTEXT("Core", "FNameCarriageReturnName", "carriage return [\\r]"));
				break;
			case EInvalidCharacters::LineFeed:
				MatchedInvalidWhitespaceChars.Add(NSLOCTEXT("Core", "FNameLineFeedName", "line feed [\\n]"));
				break;
			case EInvalidCharacters::Tab:
				MatchedInvalidWhitespaceChars.Add(NSLOCTEXT("Core", "FNameTabName", "tab"));
				break;
			default:
				MatchedInvalidChars.AppendChar(InvalidChar);
			}
			AlreadyMatchedInvalidChars.Add(InvalidChar);
		}
	}

	const bool bHasMatchedInvalidWhitespaceChars = MatchedInvalidWhitespaceChars.Num() > 0;
	const bool bHasMatchedInvalidChars = MatchedInvalidChars.Len() > 0;

	if (bHasMatchedInvalidChars || bHasMatchedInvalidWhitespaceChars)
	{
		if (OutReason)
		{
			FText InvalidWhiteSpaceCharsFText =
				FText::Join(FText::FromString(TEXT(", ")), MatchedInvalidWhitespaceChars);
			FFormatNamedArguments Args;
			Args.Add(TEXT("ErrorCtx"), InErrorCtx ? *InErrorCtx : NSLOCTEXT("Core", "NameDefaultErrorCtx", "Name"));
			
			if (bHasMatchedInvalidWhitespaceChars)
			{
				Args.Add(TEXT("InvalidWhitespaceChars"), InvalidWhiteSpaceCharsFText);
				*OutReason =  FText::Format(NSLOCTEXT("Core", "NameContainsInvalidWhitespaceCharacters", "{ErrorCtx} may not contain whitespace characters ({InvalidWhitespaceChars})"), Args);	
			}
			if (bHasMatchedInvalidChars)
			{
				FString InvalidCharacters;
				
				for (TCHAR ch : MatchedInvalidChars)
				{
					if (FTextChar::IsWhitespace(ch))
					{
						InvalidCharacters.Appendf(TEXT(" U+%04X"), ch);
						continue;
					}
					InvalidCharacters.Appendf(TEXT(" %c"), ch);
				}
				Args.Add(TEXT("InvalidCharacters"), FText::FromString(InvalidCharacters));

				*OutReason = bHasMatchedInvalidWhitespaceChars ?
					FText::Format(NSLOCTEXT("Core", "NameContainsInvalidMixedCharacters", "{ErrorCtx} may not contain whitespace characters ({InvalidWhitespaceChars}) or the following characters:{InvalidCharacters}"), Args) :
					FText::Format(NSLOCTEXT("Core", "NameContainsInvalidNonwhitespaceCharacters", "{ErrorCtx} may not contain the following characters:{InvalidCharacters}"), Args) ;
			}

		}
		return false;
	}

	return true;
}

bool FName::IsValidObjectName(FText& OutReason) const
{
	return IsValidXName(*this, INVALID_OBJECTNAME_CHARACTERS, &OutReason);
}

bool FName::IsValidGroupName(FText& OutReason, bool bIsGroupName) const
{
	return IsValidXName(*this, INVALID_LONGPACKAGE_CHARACTERS, &OutReason);
}

FString FName::SanitizeWhitespace(const FString& FNameString)
{
	FString ReturnStr;

	for (const TCHAR CurrentChar : FNameString)
	{
		if (!FTextChar::IsWhitespace(CurrentChar) || (CurrentChar == ' ' || CurrentChar =='\t'))
		{
			ReturnStr.AppendChar(CurrentChar);
		}	
	}

	return ReturnStr;
}

FWideStringBuilderBase& operator<<(FWideStringBuilderBase& Builder, FNameEntryId Id)
{
	FName::GetEntry(Id)->AppendNameToString(Builder);
	return Builder;
}

FUtf8StringBuilderBase& operator<<(FUtf8StringBuilderBase& Builder, FNameEntryId Id)
{
	FName::GetEntry(Id)->AppendNameToString(Builder);
	return Builder;
}

#if WITH_TESTS

#if WITH_LOW_LEVEL_TESTS
#define NAME_CHECK CHECK
#else
#define NAME_CHECK check
#endif

template <typename CharType, int N>
void CheckLazyName(const CharType(&Literal)[N])
{
	NAME_CHECK(FName(Literal) == FLazyName(Literal));
	NAME_CHECK(FLazyName(Literal) == FName(Literal));
	NAME_CHECK(FLazyName(Literal) == FLazyName(Literal));
	NAME_CHECK(FName(Literal) == FLazyName(Literal).Resolve());

	CharType Literal2[N];
	FMemory::Memcpy(Literal2, Literal);
	NAME_CHECK(FLazyName(Literal) == FLazyName(Literal2));

	constexpr ESearchCase::Type ComparisonMode =
#if WITH_CASE_PRESERVING_NAME
		ESearchCase::CaseSensitive;
#else
		ESearchCase::IgnoreCase;
#endif

	NAME_CHECK(WriteToString<64>(FLazyName(Literal).Resolve()).ToView().Equals(WriteToString<64>(Literal).ToView(), ComparisonMode));
}

static void TestNameBatch();

struct FNameAutoTestTruncateHelper
{
	static constexpr const TCHAR TestStr[] = TEXT("ABC_2147483646");
	static constexpr const uint32 TestStrLen = UE_ARRAY_COUNT(TestStr) - 1;

	template <uint32 RequiredLen>
	static void TestInner()
	{
		TCHAR CanaryValue;
		FMemory::Memset(&CanaryValue, 0xaa, sizeof(CanaryValue));

		// Make a buffer of RequiredLen characters, plus a null terminator.
		// Place a Canary after it in memory, to catch buffer overflow.
		struct { TCHAR Data[RequiredLen + 1]; TCHAR Canary; } Buffer;
		FMemory::Memset(&Buffer, 0xaa, sizeof(Buffer));

		uint32 Len = FName(TestStr).ToStringTruncate(Buffer.Data);
		if constexpr (RequiredLen >= TestStrLen)
		{
			// Whole string should be returned
			NAME_CHECK(Len == TestStrLen);
		}
		else
		{
			// String should be truncated
			NAME_CHECK(Len == RequiredLen);
		}

		for (uint32 Index = 0; Index <= RequiredLen; ++Index)
		{
			if (Index < Len)
			{
				NAME_CHECK(Buffer.Data[Index] == TestStr[Index]);
			}
			else if (Index == Len)
			{
				NAME_CHECK(Buffer.Data[Index] == 0); // null terminator
			}
			else
			{
				NAME_CHECK(Buffer.Data[Index] == CanaryValue); // should not have been written
			}
		}

		NAME_CHECK(Buffer.Canary == CanaryValue);
	}

	template <std::size_t... Ints>
	static void TestOuter(std::index_sequence<Ints...>)
	{
		(TestInner<Ints>(), ...);
	}

	static void Run()
	{
		TestOuter(std::make_index_sequence<TestStrLen + 4>());
	}
};

TEST_CASE_NAMED(FNameTest, "System::Core::Name", "[ApplicationContextMask][SmokeFilter]")
{
	NAME_CHECK(FNameHash::IsAnsiNone("None", 4) == 1);
	NAME_CHECK(FNameHash::IsAnsiNone("none", 4) == 1);
	NAME_CHECK(FNameHash::IsAnsiNone("NONE", 4) == 1);
	NAME_CHECK(FNameHash::IsAnsiNone("nOnE", 4) == 1);
	NAME_CHECK(FNameHash::IsAnsiNone("None", 5) == 0);
	NAME_CHECK(FNameHash::IsAnsiNone(TEXT("None"), 4) == 0);
	NAME_CHECK(FNameHash::IsAnsiNone("nono", 4) == 0);
	NAME_CHECK(FNameHash::IsAnsiNone("enon", 4) == 0);

	const FName AutoTest_1("AutoTest_1");
	const FName autoTest_1("autoTest_1");
	const FName autoTeSt_1("autoTeSt_1");
	const FName AutoTest_2(TEXT("AutoTest_2"));
	const FName AutoTestB_2(TEXT("AutoTestB_2"));

	NAME_CHECK(AutoTest_1.GetNumber() == NAME_EXTERNAL_TO_INTERNAL(1));
	NAME_CHECK(autoTest_1.GetNumber() == NAME_EXTERNAL_TO_INTERNAL(1));
	NAME_CHECK(autoTeSt_1.GetNumber() == NAME_EXTERNAL_TO_INTERNAL(1));
	NAME_CHECK(AutoTest_2.GetNumber() == NAME_EXTERNAL_TO_INTERNAL(2));
	NAME_CHECK(AutoTestB_2.GetNumber() == NAME_EXTERNAL_TO_INTERNAL(2));

	NAME_CHECK(AutoTest_1 != AutoTest_2);
	NAME_CHECK(AutoTest_1 == autoTest_1);
	NAME_CHECK(AutoTest_1 == autoTeSt_1);

	NAME_CHECK(AutoTest_1.ToUnstableInt() == autoTest_1.ToUnstableInt());
	NAME_CHECK(AutoTest_1.ToUnstableInt() != AutoTest_2.ToUnstableInt());

	TCHAR Buffer[FName::StringBufferSize];

#if WITH_CASE_PRESERVING_NAME
	NAME_CHECK(!FCString::Strcmp(*AutoTest_1.ToString(), TEXT("AutoTest_1")));
	NAME_CHECK(!FCString::Strcmp(*autoTest_1.ToString(), TEXT("autoTest_1")));
	NAME_CHECK(!FCString::Strcmp(*autoTeSt_1.ToString(), TEXT("autoTeSt_1")));
	NAME_CHECK(!FCString::Strcmp(*AutoTestB_2.ToString(), TEXT("AutoTestB_2")));
	
	NAME_CHECK(FName("ABC").ToString(Buffer) == 3);
	NAME_CHECK(!FCString::Strcmp(Buffer, TEXT("ABC")));
	NAME_CHECK(FName("abc").ToString(Buffer) == 3);
	NAME_CHECK(!FCString::Strcmp(Buffer, TEXT("abc")));
	NAME_CHECK(FName(TEXT("abc")).ToString(Buffer) == 3);
	NAME_CHECK(!FCString::Strcmp(Buffer, TEXT("abc")));
	NAME_CHECK(FName("ABC_0").ToString(Buffer) == 5);
	NAME_CHECK(!FCString::Strcmp(Buffer, TEXT("ABC_0")));
	NAME_CHECK(FName("ABC_10").ToString(Buffer) == 6);
	NAME_CHECK(!FCString::Strcmp(Buffer, TEXT("ABC_10")));	
#endif

	NAME_CHECK(autoTest_1.GetComparisonIndex() == AutoTest_2.GetComparisonIndex());
	NAME_CHECK(autoTest_1.GetPlainNameString() == AutoTest_1.GetPlainNameString());
	NAME_CHECK(autoTest_1.GetPlainNameString() == AutoTest_2.GetPlainNameString());
	NAME_CHECK(*AutoTestB_2.GetPlainNameString() != *AutoTest_2.GetPlainNameString());
	NAME_CHECK(AutoTestB_2.GetNumber() == AutoTest_2.GetNumber());
	NAME_CHECK(autoTest_1.GetNumber() != AutoTest_2.GetNumber());
	NAME_CHECK(AutoTest_1.GetNumber() == autoTest_1.GetNumber());

	NAME_CHECK(FCStringAnsi::Strlen("None") == FName().GetStringLength());
	NAME_CHECK(FCStringAnsi::Strlen("ABC") == FName("ABC").GetStringLength());
	NAME_CHECK(FCStringAnsi::Strlen("ABC_0") == FName("ABC_0").GetStringLength());
	NAME_CHECK(FCStringAnsi::Strlen("ABC_9") == FName("ABC_9").GetStringLength());
	NAME_CHECK(FCStringAnsi::Strlen("ABC_10") == FName("ABC_10").GetStringLength());
	NAME_CHECK(FCStringAnsi::Strlen("ABC_2000000000") == FName("ABC_2000000000").GetStringLength());
	NAME_CHECK(FCStringAnsi::Strlen("ABC_4000000000") == FName("ABC_4000000000").GetStringLength());

	const FName NullName(static_cast<ANSICHAR*>(nullptr));
	NAME_CHECK(NullName.IsNone());
	NAME_CHECK(NullName == FName(static_cast<WIDECHAR*>(nullptr)));
	NAME_CHECK(NullName == FName(NAME_None));
	NAME_CHECK(NullName == FName());
	NAME_CHECK(NullName == FName(""));
	NAME_CHECK(NullName == FName(TEXT("")));
	NAME_CHECK(NullName == FName("None"));
	NAME_CHECK(NullName == FName("none"));
	NAME_CHECK(NullName == FName("NONE"));
	NAME_CHECK(NullName == FName(TEXT("None")));
	NAME_CHECK(FName().ToEName());
	NAME_CHECK(*FName().ToEName() == NAME_None);
	NAME_CHECK(NullName.GetComparisonIndex().ToUnstableInt() == 0);

	// Numbered nones
	FName None_7 = FName("None_7");
	NAME_CHECK(None_7.ToString() == "None_7");
	NAME_CHECK(None_7.GetComparisonIndex() == FNameEntryId::FromEName(EName::None));
	NAME_CHECK(None_7.GetNumber() == NAME_EXTERNAL_TO_INTERNAL(7));
	NAME_CHECK(FName(EName::None, NAME_EXTERNAL_TO_INTERNAL(7)) == None_7);
	NAME_CHECK(FName(FName(), NAME_EXTERNAL_TO_INTERNAL(7)) == None_7);
	NAME_CHECK(FName("None", NAME_EXTERNAL_TO_INTERNAL(7)) == None_7);
	NAME_CHECK(FName(FNameEntryId::FromEName(EName::None), FNameEntryId::FromEName(EName::None), NAME_EXTERNAL_TO_INTERNAL(7)) == None_7);
	NAME_CHECK(!None_7.IsNone());

	// "_[num]" is considered None to keep semantics of old implementation, unlike "None_[num]"
	NAME_CHECK(FName("_7").IsNone()); 
	NAME_CHECK(FName("_0").IsNone());
	NAME_CHECK(FName("_2147483646").IsNone());
	NAME_CHECK(!FName("_2147483647").IsNone()); // invalid number

	const FName Cylinder(NAME_Cylinder);
	NAME_CHECK(Cylinder == FName("Cylinder"));
	NAME_CHECK(Cylinder.ToEName());
	NAME_CHECK(*Cylinder.ToEName() == NAME_Cylinder);
	NAME_CHECK(Cylinder.GetPlainNameString() == TEXT("Cylinder"));

	// Test numbers
	NAME_CHECK(FName("Text_0") == FName("Text", NAME_EXTERNAL_TO_INTERNAL(0)));
	NAME_CHECK(FName("Text_1") == FName("Text", NAME_EXTERNAL_TO_INTERNAL(1)));
	NAME_CHECK(FName("Text_1_0") == FName("Text_1", NAME_EXTERNAL_TO_INTERNAL(0)));
	NAME_CHECK(FName("Text_0_1") == FName("Text_0", NAME_EXTERNAL_TO_INTERNAL(1)));
	NAME_CHECK(FName("Text_00") == FName("Text_00", NAME_NO_NUMBER_INTERNAL));
	NAME_CHECK(FName("Text_01") == FName("Text_01", NAME_NO_NUMBER_INTERNAL));

	// Test unterminated strings
	NAME_CHECK(FName("") == FName(0, "Unused"));
	NAME_CHECK(FName("Used") == FName(4, "UsedUnused"));
	NAME_CHECK(FName("Used") == FName(4, "Used"));
	NAME_CHECK(FName("Used_0") == FName(6, "Used_01"));
	NAME_CHECK(FName("Used_01") == FName(7, "Used_012"));
	NAME_CHECK(FName("Used_123") == FName(8, "Used_123456"));
	NAME_CHECK(FName("Used_123") == FName(8, "Used_123_456"));
	NAME_CHECK(FName("Used_123") == FName(8, TEXT("Used_123456")));
	NAME_CHECK(FName("Used_123") == FName(8, TEXT("Used_123_456")));
	NAME_CHECK(FName("Used_2147483646") == FName(15, TEXT("Used_2147483646123")));
	NAME_CHECK(FName("Used_2147483647") == FName(15, TEXT("Used_2147483647123")));
	NAME_CHECK(FName("Used_2147483648") == FName(15, TEXT("Used_2147483648123")));

	// Test wide strings
	FString Wide("Wide ");
	Wide[4] = 60000;
	FName WideName(*Wide);
	NAME_CHECK(WideName.GetPlainNameString() == Wide);
	NAME_CHECK(FName(*Wide).GetPlainNameString() == Wide);
	NAME_CHECK(FName(*Wide).ToString(Buffer) == 5);
	NAME_CHECK(!FCString::Strcmp(Buffer, *Wide));
	NAME_CHECK(Wide.Len() == WideName.GetStringLength());
	FString WideLong = FString::ChrN(1000, 60000);
	NAME_CHECK(FName(*WideLong).GetPlainNameString() == WideLong);


	// Check that FNAME_Find doesn't add entries
	static bool Once = true;
	if (Once)
	{
		NAME_CHECK(FName("UniqueUnicorn!!", FNAME_Find) == FName());

		// Check that FNAME_Find can find entries
		const FName UniqueName("UniqueUnicorn!!", FNAME_Add);
		NAME_CHECK(FName("UniqueUnicorn!!", FNAME_Find) == UniqueName);
		NAME_CHECK(FName(TEXT("UniqueUnicorn!!"), FNAME_Find) == UniqueName);
		NAME_CHECK(FName("UNIQUEUNICORN!!", FNAME_Find) == UniqueName);
		NAME_CHECK(FName(TEXT("UNIQUEUNICORN!!"), FNAME_Find) == UniqueName);
		NAME_CHECK(FName("uniqueunicorn!!", FNAME_Find) == UniqueName);

		Once = false;
	}

	TArray<FName> Names;
	Names.Add("FooB");
	Names.Add("FooABCD");
	Names.Add("FooABC");
	Names.Add("FooAB");
	Names.Add("FooA");
	Names.Add("FooC");
	const WIDECHAR FooWide[] = {'F', 'o', 'o', (WIDECHAR)2000, '\0'};
	Names.Add(FooWide);
	Algo::Sort(Names, FNameLexicalLess()); 

	NAME_CHECK(Names[0] == "FooA");
	NAME_CHECK(Names[1] == "FooAB");
	NAME_CHECK(Names[2] == "FooABC");
	NAME_CHECK(Names[3] == "FooABCD");
	NAME_CHECK(Names[4] == "FooB");
	NAME_CHECK(Names[5] == "FooC");
	NAME_CHECK(Names[6] == FooWide);
		
	NAME_CHECK(FName() == TEXTVIEW("None"));
	NAME_CHECK(FName() == TEXTVIEW("none"));
	NAME_CHECK(FName() == TEXTVIEW(""));
	NAME_CHECK(FName() == FStringView());
	NAME_CHECK(FName("a") == TEXTVIEW("a"));
	NAME_CHECK(FName("a") == TEXTVIEW("A"));
	NAME_CHECK(FName("A") == TEXTVIEW("a"));
	NAME_CHECK(FName("aa") != TEXTVIEW("a"));
	NAME_CHECK(FName("a") != TEXTVIEW("aa"));
	NAME_CHECK(FName("a") == TEXTVIEW("aa").Left(1)); // Unterminated
	NAME_CHECK(FName("aa") != TEXTVIEW("ab"));
	NAME_CHECK(FName("a_0") == TEXTVIEW("a_0"));
	NAME_CHECK(FName("a_1") != TEXTVIEW("a_0"));
	NAME_CHECK(FName("a_10") == TEXTVIEW("a_10"));
	NAME_CHECK(FName("a")    == TEXTVIEW("a_10").Left(1)); // Unterminated number
	NAME_CHECK(FName("a_")   == TEXTVIEW("a_10").Left(2)); // Unterminated number
	NAME_CHECK(FName("a_1")  == TEXTVIEW("a_10").Left(3)); // Unterminated number
	NAME_CHECK(FName("a_2147483646") == TEXTVIEW("a_2147483646")); // Largest valid number
	NAME_CHECK(FName("a_2147483647") == TEXTVIEW("a_2147483647")); // Smallest invalid number
	NAME_CHECK(FName("a_") == TEXTVIEW("a_"));
	NAME_CHECK(FName("_") == TEXTVIEW("_"));
	NAME_CHECK(FName("_3") == TEXTVIEW("_3")); // Special none
	NAME_CHECK(FName() == TEXTVIEW("_123")); // Special none
	NAME_CHECK(FName("_3") == TEXTVIEW("None")); // Special none
	NAME_CHECK(FName("none_1") == TEXTVIEW("none_1")); // Numbered none
	NAME_CHECK(WideName == FStringView(Wide));
	NAME_CHECK(FName("a") == ANSITEXTVIEW("a"));
	NAME_CHECK(FName("a") == ANSITEXTVIEW("A"));
	NAME_CHECK(FName("A") != ANSITEXTVIEW("b"));

	// Check truncating ToString functions
	FNameAutoTestTruncateHelper::Run();
	
	CheckLazyName("Hej");
	CheckLazyName("hej");
	CheckLazyName("HEJ");
	CheckLazyName(UTF8TEXT("Hej"));
	CheckLazyName(WIDETEXT("Hej"));
	CheckLazyName("Hej_0");
	CheckLazyName("Hej_00");
	CheckLazyName("Hej_1");
	CheckLazyName("Hej_01");
	CheckLazyName("Hej_-1");
	CheckLazyName("Hej__0");
	CheckLazyName("Hej_2147483646");
	CheckLazyName("Hej_2147483647");
	CheckLazyName("Hej_123");
	CheckLazyName("None");
	CheckLazyName("none");
	CheckLazyName("None_0");
	CheckLazyName("None_1");

	NAME_CHECK(FLazyName("Hej") == FLazyName(UTF8TEXT("Hej")));
	NAME_CHECK(FLazyName("Hej") == FLazyName(WIDETEXT("Hej")));
	NAME_CHECK(FLazyName(UTF8TEXT("Hej")) == FLazyName("Hej"));
	NAME_CHECK(FLazyName(UTF8TEXT("Hej")) == FLazyName(WIDETEXT("Hej")));
	NAME_CHECK(FLazyName(WIDETEXT("Hej")) == FLazyName("Hej"));
	NAME_CHECK(FLazyName(WIDETEXT("Hej")) == FLazyName(UTF8TEXT("Hej")));

	TestNameBatch();

#if 0
	// Check hash table growth still yields the same unique FName ids
	static int32 OverflowAtLeastTwiceCount = 4 * FNamePoolInitialSlotsPerShard * FNamePoolShards;
	TArray<FNameEntryId> Ids;
	for (int I = 0; I < OverflowAtLeastTwiceCount; ++I)
	{
		FNameEntryId Id = FName(*FString::Printf(TEXT("%d"), I)).GetComparisonIndex();
		Ids.Add(Id);
	}

	for (int I = 0; I < OverflowAtLeastTwiceCount; ++I)
	{
		FNameEntryId Id = FName(*FString::Printf(TEXT("%d"), I)).GetComparisonIndex();
		FNameEntryId OldId = Ids[I];

		while (Id != OldId)
		{
			Id = FName(*FString::Printf(TEXT("%d"), I)).GetComparisonIndex();
		}
		NAME_CHECK(Id == OldId);
	}
#endif

	{
		TNameAtomicBitSet<1024> TestSet;
		uint32 SetBits[] = { 0, 3, 34, 35, 53, 65, 523, 534, 654, 1001, 1023 };
		for (uint32 Index : SetBits)
		{
			const bool bWasSet = TestSet.SetBitAtomic(Index);
			NAME_CHECK(bWasSet == false);
		}

		const bool b32WasSet = TestSet.SetBitAtomic(34);
		NAME_CHECK(b32WasSet);

		uint32 Expected = 0;
		TestSet.ForEachSetBit([&](uint32 Index)
		{
			NAME_CHECK(Index == SetBits[Expected]);
			++Expected;
		});
		NAME_CHECK(Expected == UE_ARRAY_COUNT(SetBits));

		const bool b1023WasSet = TestSet.ClearBitAtomic(1023);
		NAME_CHECK(b1023WasSet == true);
		TestSet.ForEachSetBit([&](uint32 Index)
		{
			NAME_CHECK(Index != 1023);
		});
	}
}

#undef NAME_CHECK

#endif // WITH_TESTS


/*-----------------------------------------------------------------------------
	FNameEntry implementation.
-----------------------------------------------------------------------------*/

void FNameEntry::Write( FArchive& Ar ) const
{
	// This path should be unused - since FNameEntry structs are allocated with a dynamic size, we can only save them. Use FNameEntrySerialized to read them back into an intermediate buffer.
	checkf(!Ar.IsLoading(), TEXT("FNameEntry does not support reading from an archive. Serialize into a FNameEntrySerialized and construct a FNameEntry from that."));

	// Convert to our serialized type
	FNameEntrySerialized EntrySerialized(*this);
	Ar << EntrySerialized;
}

static_assert(PLATFORM_LITTLE_ENDIAN, "FNameEntrySerialized serialization needs updating to support big-endian platforms!");

FArchive& operator<<(FArchive& Ar, FNameEntrySerialized& E)
{
	if (Ar.IsLoading())
	{
		const int64 MaxSerialize = Ar.GetMaxSerializeSize();
		const int32 MaxSize = MaxSerialize > 0 ? static_cast<int32>(FMath::Min(MaxSerialize, int64(NAME_SIZE))) : NAME_SIZE;

		// for optimization reasons, we want to keep pure Ansi strings as Ansi for initializing the name entry
		// (and later the FName) to stop copying in and out of TCHARs
		int32 StringLen;
		Ar << StringLen;

		if ((StringLen < -MaxSize) | (StringLen > MaxSize))
		{
			Ar.SetCriticalError();
			UE_LOGF(LogUnrealNames, Error, "String is too long");
			return Ar;
		}
		
		// negative stringlen means it's a wide string
		E.bIsWide = StringLen < 0;
		if (E.bIsWide)
		{
			StringLen = -StringLen;

			// get the pointer to the wide array 
			WIDECHAR* WideName = const_cast<WIDECHAR*>(E.GetWideName());

			// read in the UCS2CHAR string
			auto Sink = StringMemoryPassthru<UCS2CHAR>(WideName, StringLen, StringLen);
			Ar.Serialize(Sink.Get(), StringLen * sizeof(UCS2CHAR));
			Sink.Apply();
		}
		else
		{
			// ansi strings can go right into the AnsiBuffer
			ANSICHAR* AnsiName = const_cast<ANSICHAR*>(E.GetAnsiName());
			Ar.Serialize(AnsiName, StringLen);
		}

		uint16 DummyHashes[2];
		uint32 SkipPastHashBytes = (Ar.UEVer() >= VER_UE4_NAME_HASHES_SERIALIZED) * sizeof(DummyHashes);
		Ar.Serialize(&DummyHashes, SkipPastHashBytes);
	}
	else
	{
		// These hashes are no longer used. They're only kept to maintain serialization format.
		// Please remove them if you ever change serialization format.
		FString Str = E.GetPlainNameString();
		Ar << Str;
		Ar << E.NonCasePreservingHash;
		Ar << E.CasePreservingHash;
	}

	return Ar;
}

void FNameEntry::DebugDump(FOutputDevice& Out) const
{
#if UE_FNAME_OUTLINE_NUMBER
	if (IsNumbered())
	{
		TCHAR Buffer[NAME_SIZE];
		const FNameEntry& BaseEntry = GetNamePool().Resolve(GetNumberedName().Id);
		BaseEntry.GetName(Buffer);
		Out.Logf(TEXT("%s_%d"), Buffer, NAME_INTERNAL_TO_EXTERNAL(GetNumberedName().Number));
	}
	else
#endif
	{
		TCHAR Buffer[NAME_SIZE];
		GetName(Buffer);
		Out.Log(Buffer);
	}
}

FNameEntryId FNameEntryId::FromValidEName(EName Ename)
{
	return GetNamePool().Find(Ename);
}

#if UE_FNAME_OUTLINE_NUMBER
FName FName::CreateNumberedName(FNameEntryId ComparisonId, FNameEntryId DisplayId, int32 InInternalNumber)
{
	// Users must only pass in base/plain ids, not numbered ones.
#if WITH_CASE_PRESERVING_NAME
	checkName(ResolveEntry(DisplayId)->ComparisonId == ComparisonId);
#endif
	checkName(ResolveEntry(ComparisonId)->IsNumbered() == false);

	return FNameHelper::MakeWithInternalNumber(FNameEntryIds{ ComparisonId, DisplayId }, FNAME_Add, InInternalNumber);
}
#endif // UE_FNAME_OUTLINE_NUMBER

FNameEntryId FNameEntryId::FromValidENamePostInit(EName Ename)
{
	return GetNamePoolPostInit().Find(Ename);
}

void FName::TearDown()
{
	check(IsInGameThread());

	if (bNamePoolInitialized)
	{
		GetNamePoolPostInit().~FNamePool();
		bNamePoolInitialized = false;
	}
}

FName FLazyName::Resolve() const
{
	// Make a stack copy to ensure thread-safety
	FLiteralOrName Copy = Either;

	if (Copy.IsName())
	{
		FNameEntryId ComparisonId = Copy.GetComparisonId();
		FNameEntryId DisplayId = Copy.GetDisplayId();
#if UE_FNAME_OUTLINE_NUMBER
		return FNameHelper::MakeWithInternalNumber(FNameEntryIds{ ComparisonId, DisplayId }, FNAME_Add, Number);
#else // UE_FNAME_OUTLINE_NUMBER
		return FName(ComparisonId, DisplayId, Number);
#endif // UE_FNAME_OUTLINE_NUMBER
	}

	// Resolve to FName but throw away the number part
	FName FullName;
	switch (LiteralType)
	{
	case ELiteralType::AnsiLiteral:
		FullName = FName(Copy.AnsiLiteral);
		break;
	case ELiteralType::Utf8Literal:
		FullName = FName(Copy.Utf8Literal);
		break;
	case ELiteralType::WideLiteral:
		FullName = FName(Copy.WideLiteral);
		break;
	default: checkNoEntry();
		break;
	}
	FNameEntryId ComparisonId = FullName.GetComparisonIndex();
	FNameEntryId DisplayId = FullName.GetDisplayIndex();

	// Deliberately unsynchronized write of word-sized int, ok if multiple threads resolve same lazy name
	Either.PackedName = FLiteralOrName::PackName(ComparisonId, DisplayId);

#if UE_FNAME_OUTLINE_NUMBER
	return FNameHelper::MakeWithInternalNumber(FNameEntryIds{ ComparisonId, DisplayId}, FNAME_Add, Number);
#else // UE_FNAME_OUTLINE_NUMBER
	return FName(ComparisonId, DisplayId, Number);
#endif // UE_FNAME_OUTLINE_NUMBER
}

CORE_API FString FLazyName::ToString() const
{
	return Resolve().ToString();
}

CORE_API FUtf8String FLazyName::ToUtf8String() const
{
	return Resolve().ToUtf8String();
}

template <typename CharType>
uint32 FLazyName::CallParseNumber(const CharType* Literal, int32 Len)
{
	return UE::Core::Private::ParseNumberFromName(Literal, Len);
}

template CORE_API uint32 FLazyName::CallParseNumber(const ANSICHAR* Literal, int32 Len);
template CORE_API uint32 FLazyName::CallParseNumber(const UTF8CHAR* Literal, int32 Len);
template CORE_API uint32 FLazyName::CallParseNumber(const WIDECHAR* Literal, int32 Len);

bool FLazyName::UEOpEquals(const FLazyName& Rhs) const
{
	// Copy these because a concurrent Resolve() can modify them during this function.
	FLiteralOrName LhsEither = Either;
	FLiteralOrName RhsEither = Rhs.Either;

	// If we have started creating FNames we might as well resolve and cache both lazy names
	if (LhsEither.IsName() || RhsEither.IsName())
	{
		return Resolve() == Rhs.Resolve();
	}

	// Literal pointer comparison, can ignore width
	if (LhsEither.AnsiLiteral == RhsEither.AnsiLiteral)
	{
		return true;
	}

	ELiteralType LhsType = LiteralType;
	ELiteralType RhsType = Rhs.LiteralType;

	// Always compare with LhsType <= RhsType to simplify the branches below.
	if (RhsType < LhsType)
	{
		Swap(LhsType, RhsType);
		Swap(LhsEither, RhsEither);
	}

	if (RhsType == ELiteralType::WideLiteral)
	{
		switch (LhsType)
		{
		case ELiteralType::AnsiLiteral:
			return FPlatformString::Stricmp(LhsEither.AnsiLiteral, RhsEither.WideLiteral) == 0;
		case ELiteralType::WideLiteral:
			return FPlatformString::Stricmp(LhsEither.WideLiteral, RhsEither.WideLiteral) == 0;
		case ELiteralType::Utf8Literal:
			return FPlatformString::Stricmp(*WriteToString<FName::StringBufferSize>(LhsEither.Utf8Literal), RhsEither.WideLiteral) == 0;
		}
	}
	return (RhsType == ELiteralType::Utf8Literal) ?
		FPlatformString::Stricmp(LhsEither.Utf8Literal, RhsEither.Utf8Literal) == 0 :
		FPlatformString::Stricmp(LhsEither.AnsiLiteral, RhsEither.AnsiLiteral) == 0;
}

/*-----------------------------------------------------------------------------
	FName batch serialization
-----------------------------------------------------------------------------*/

FORCEINLINE FNameEntryId FDisplayNameEntryId::ToDisplayId() const
{
	return GetDisplayId();
}

FORCEINLINE FDisplayNameEntryId FDisplayNameEntryId::FromComparisonId(FNameEntryId ComparisonId)
{
	return FDisplayNameEntryId(ComparisonId, ComparisonId);
}

#if WITH_CASE_PRESERVING_NAME

FNameEntryId FDisplayNameEntryId::GetLoadedComparisonId() const
{
	checkf(SameIds(), TEXT("SetLoadedComparisonId() should've been called at this point. The display id should not have been created yet. "));
	return GetComparisonId();
}

void FDisplayNameEntryId::SetLoadedDifferentDisplayId(FNameEntryId Id)
{
	static_assert((FNameEntryIdMask & DifferentIdsFlag) == 0u);
	check((Id.ToUnstableInt() & DifferentIdsFlag) == 0u);
	Value = Id.ToUnstableInt() | DifferentIdsFlag;
}

void FDisplayNameEntryId::SetLoadedComparisonId(FNameEntryId Id)
{
	check((Id.ToUnstableInt() & DifferentIdsFlag) == 0u);
	Value = Id.ToUnstableInt();
}

#else

FORCEINLINE void FDisplayNameEntryId::SetLoadedComparisonId(FNameEntryId InId)
{
	Id = InId;
}

#endif


constexpr bool CanCastUtf16ToWideCharWithoutConversion()
{
#if PLATFORM_LITTLE_ENDIAN
	return sizeof(WIDECHAR) == sizeof(UTF16CHAR);
#else
	return false;
#endif
}


/**
 * FNameStringView sibling with UTF16 Little-Endian wide strings instead of WIDECHAR 
 *
 * View into serialized data instead of how it will be stored in memory once loaded.
 */
struct FNameSerializedView
{
	FNameSerializedView(const ANSICHAR* InStr, uint32 InLen)
	: Ansi(InStr)
	, Len(InLen)
	, bIsUtf16(false)
	{}
	
	FNameSerializedView(const UTF16CHAR* InStr, uint32 InLen)
	: Utf16(InStr)
	, Len(InLen)
	, bIsUtf16(true)
	{}

	FNameSerializedView(const uint8* InData, uint32 InLen, bool bInUtf16)
	: Data(InData)
	, Len(InLen)
	, bIsUtf16(bInUtf16)
	{}

	union
	{
		const uint8* Data;
		const ANSICHAR* Ansi;
		const UTF16CHAR* Utf16;
	};

	uint32 Len;
	bool bIsUtf16;

	uint32 NumBytes() const
	{
		return bIsUtf16 ? sizeof(UTF16CHAR) * Len : sizeof(ANSICHAR) * Len; 
	}

	FORCEINLINE FNameStringView CastToNameView() const
	{
		check(CanCastUtf16ToWideCharWithoutConversion());
		return FNameStringView(Data, Len, /* bIsWide */ bIsUtf16);
	}
};

static uint8* AddUninitializedBytes(TArray<uint8>& Out, uint32 Bytes)
{
	uint32 OldNum = Out.AddUninitialized(Bytes);
	return Out.GetData() + OldNum;
}

template<typename T>
static T* AddUninitializedElements(TArray<uint8>& Out, uint32 Num)
{
	check(Out.Num() %  alignof(T) == 0);
	return reinterpret_cast<T*>(AddUninitializedBytes(Out, Num * sizeof(T)));
}

template<typename T>
static void AddValue(TArray<uint8>& Out, T Value)
{
	*AddUninitializedElements<T>(Out, 1) = Value;
}

static uint32 GetRequiredUtf16Padding(const uint8* Ptr)
{
	return UPTRINT(Ptr) & 1u;
}

struct FSerializedNameHeader
{
	FSerializedNameHeader() {}
	FSerializedNameHeader(uint32 Len, bool bIsUtf16)
	{
		static_assert(NAME_SIZE < 0x8000u, "");
		check(Len < NAME_SIZE);

		Data[0] = uint8(bIsUtf16) << 7 | static_cast<uint8>(Len >> 8);
		Data[1] = static_cast<uint8>(Len);
	}

	uint8 IsUtf16() const
	{
		return Data[0] & 0x80u;
	}

	uint32 Len() const
	{
		return ((Data[0] & 0x7Fu) << 8) + Data[1];
	}

	uint32 NumBytes() const
	{
		return IsUtf16() ? sizeof(UTF16CHAR) * Len() : sizeof(ANSICHAR) * Len(); 
	}

	uint8 Data[2];
};

FNameSerializedView LoadNameHeader(const uint8*& InOutIt)
{
	const FSerializedNameHeader& Header = *reinterpret_cast<const FSerializedNameHeader*>(InOutIt);
	const uint8* NameData = InOutIt + sizeof(FSerializedNameHeader);
	const uint32 Len = Header.Len();

	if (Header.IsUtf16())
	{
		NameData += GetRequiredUtf16Padding(NameData);
		InOutIt = NameData + Len * sizeof(UTF16CHAR);
		return FNameSerializedView(NameData, Len, /* UTF16 */ true);
	}
	else
	{
		InOutIt = NameData + Len * sizeof(ANSICHAR);
		return FNameSerializedView(NameData, Len, /* UTF16 */ false);
	}
}

#if ALLOW_NAME_BATCH_SAVING

static void SaveHeaderAndName(TArray<uint8>& Out, FNameStringView Name)
{
	// Align to UTF16CHAR after header
	uint32 PadBytes	= Name.bIsWide ? (Out.Num() + sizeof(FSerializedNameHeader)) % sizeof(UTF16CHAR) : 0;
	uint32 NameBytes = Name.BytesWithoutTerminator();

	uint8* HeaderData = AddUninitializedBytes(Out, sizeof(FSerializedNameHeader) + PadBytes + NameBytes);
	uint8* PadData = HeaderData + sizeof(FSerializedNameHeader);
	uint8* NameData = PadData + PadBytes;

	new (HeaderData) FSerializedNameHeader(Name.Len, Name.bIsWide);
	FMemory::Memzero(PadData, PadBytes);
	FMemory::Memcpy(NameData, Name.Data, NameBytes);
}

void SaveNameBatch(TConstArrayView<FDisplayNameEntryId> Names, TArray<uint8>& OutNameData, TArray<uint8>& OutHashData)
{
	OutNameData.Empty(/* average bytes per name guesstimate */ 40 * Names.Num());
	OutHashData.Empty((/* hash version */ 1 + Names.Num()) * sizeof(uint64));

	// Save hash algorithm version
	AddValue(OutHashData, INTEL_ORDER64(FNameHash::AlgorithmId));

	// Save names and hashes
	FNameBuffer CustomDecodeBuffer;
	for (FDisplayNameEntryId EntryId : Names)
	{
		FNameStringView Name = GetNamePoolPostInit().Resolve(EntryId.ToDisplayId()).MakeView(CustomDecodeBuffer);

		SaveHeaderAndName(OutNameData, Name);

		AddValue(OutHashData, GenerateLowerCaseHash(Name));
	}
}

void SaveNameBatch(TConstArrayView<FDisplayNameEntryId> Names, FArchive& Ar)
{
	// Save Num and bail if zero
	uint32 Num = Names.Num();
	Ar << Num;

	if (Num == 0)
	{
		return;
	}

	// Prepare hashes, headers and strings to be saved
	TArray<uint64> Hashes;
	TArray<FSerializedNameHeader> Headers;
	TArray<uint8> Strings;
	Hashes.Reserve(Num);
	Headers.Reserve(Num);
	Strings.Reserve(Num * 40 /* guesstimate */);

	FNameBuffer Buffer;
	for (FDisplayNameEntryId EntryId : Names)
	{
		FNameStringView Name = GetNamePoolPostInit().Resolve(EntryId.ToDisplayId()).MakeView(Buffer);

		Hashes.Add(GenerateLowerCaseHash(Name));
		Headers.Add(FSerializedNameHeader(Name.Len, Name.bIsWide));
		Strings.Append(reinterpret_cast<const uint8*>(Name.Data), Name.BytesWithoutTerminator());
	}

	// Save metadata and data 
	uint32 NumStringBytes = Strings.Num();
	Ar << NumStringBytes;

	uint64 HashVersion = FNameHash::AlgorithmId;
	Ar << HashVersion;

	Ar.Serialize(Hashes.GetData(), Num * Hashes.GetTypeSize());
	Ar.Serialize(Headers.GetData(), Num * Headers.GetTypeSize());
	Ar.Serialize(Strings.GetData(), Strings.Num());
}

#endif // WITH_EDITOR

FORCENOINLINE void ReserveNameBatch(uint32 NameDataBytes, uint32 HashDataBytes)
{
	uint32 NumEntries = HashDataBytes / sizeof(uint64) - 1;
	// Add 20% slack to reduce probing costs
	auto AddSlack = [](uint64 In){ return static_cast<uint32>(In * 6 / 5); };
	GetNamePoolPostInit().Reserve(AddSlack(NameDataBytes), AddSlack(NumEntries));
}

static FDisplayNameEntryId BatchLoadNameWithoutHash(const UTF16CHAR* Str, uint32 Len)
{
	WIDECHAR Temp[NAME_SIZE];
	for (uint32 Idx = 0; Idx < Len; ++Idx)
	{
		Temp[Idx] = INTEL_ORDER16(Str[Idx]);
	}

	FNameStringView Name(Temp, Len);
	FNameHash Hash = HashName<ENameCase::IgnoreCase>(Name);
	return GetNamePoolPostInit().StoreValue(FNameComparisonValue(Name, Hash));
}

static FDisplayNameEntryId BatchLoadNameWithoutHash(const ANSICHAR* Str, uint32 Len)
{
	FNameStringView Name(Str, Len);
	FNameHash Hash = HashName<ENameCase::IgnoreCase>(Name);
	return GetNamePoolPostInit().StoreValue(FNameComparisonValue(Name, Hash));
}

static FDisplayNameEntryId BatchLoadNameWithoutHash(const FNameSerializedView& Name)
{
	return Name.bIsUtf16 ? BatchLoadNameWithoutHash(Name.Utf16, Name.Len)
						 : BatchLoadNameWithoutHash(Name.Ansi, Name.Len);
}

template<typename CharType>
FDisplayNameEntryId BatchLoadNameWithHash(const CharType* Str, uint32 Len, uint64 InHash)
{
	FNameStringView Name(Str, Len);
	FNameHash Hash(Str, Len, InHash);
	checkfSlow(Hash == HashName<ENameCase::IgnoreCase>(Name), TEXT("Precalculated hash was wrong"));
	return GetNamePoolPostInit().StoreValue(FNameComparisonValue(Name, Hash));
}

static FDisplayNameEntryId BatchLoadNameWithHash(const FNameSerializedView& InName, uint64 InHash)
{
	if (InName.bIsUtf16)
	{
		// Wide names and hashes are currently stored as UTF16 Little-Endian 
		// regardless of target architecture. 
#if PLATFORM_LITTLE_ENDIAN
		if (sizeof(UTF16CHAR) == sizeof(WIDECHAR))
		{
			return BatchLoadNameWithHash(reinterpret_cast<const WIDECHAR*>(InName.Utf16), InName.Len, InHash);
		}
#endif

		return BatchLoadNameWithoutHash(InName.Utf16, InName.Len);
	}
	else
	{
		return BatchLoadNameWithHash(InName.Ansi, InName.Len, InHash);
	}
}

static void LoadInterleavedNameBatchInInputOrder(TArray<FDisplayNameEntryId>& Out, TArrayView<const uint8> Names)
{
	TArray<FDisplayNameEntryId>::TIterator OutIt(Out);
	const uint8* NameIt = Names.GetData();
	const uint8* NameEnd = Names.GetData() + Names.Num();

	while (NameIt < NameEnd)
	{
		FNameSerializedView Name = LoadNameHeader(/* in-out */ NameIt);
		*OutIt++ = BatchLoadNameWithoutHash(Name);
	}

	check(NameIt == NameEnd);
	check(OutIt.GetIndex() == Out.Num());
}

static void LoadInterleavedNameBatchInInputOrder(TArray<FDisplayNameEntryId>& Out, TArrayView<const uint64> Hashes, TArrayView<const uint8> Names)
{
	check(Hashes.Num() == Out.Num());

	TArray<FDisplayNameEntryId>::TIterator OutIt(Out);
	const uint8* NameIt = Names.GetData();

	for (uint64 Hash : Hashes)
	{
		check(NameIt < Names.end());
		FNameSerializedView Name = LoadNameHeader(/* in-out */ NameIt);
		*OutIt++ = BatchLoadNameWithHash(Name, INTEL_ORDER64(Hash));
	}

	check(NameIt == Names.end());
}

static bool CanUseSavedHashes(uint64 HashVersion)
{
	return HashVersion == FNameHash::AlgorithmId && CanCastUtf16ToWideCharWithoutConversion();
}

static FNameSerializedView LoadName(FArchive& Ar, FSerializedNameHeader Header, uint8* OutNameBuffer)
{
	FNameSerializedView Name(OutNameBuffer, Header.Len(), !!Header.IsUtf16());
	Ar.Serialize(OutNameBuffer, Name.NumBytes());
	return Name;
}

static FNameSerializedView LoadHeaderAndName(FArchive& Ar, uint8* OutNameBuffer)
{
	FSerializedNameHeader Header;
	Ar.Serialize(&Header, sizeof(Header));
	return LoadName(Ar, Header, OutNameBuffer);
}

static void LoadSeparatedNameBatchInInputOrder(	TArray<FDisplayNameEntryId>& Out,
												const uint64* Hashes,
												TArrayView<const FSerializedNameHeader> Headers, 
												TArrayView<const uint8> Strings)
{
	check(Out.Num() == Headers.Num());
	TArray<FDisplayNameEntryId>::TIterator OutIt(Out);
	const uint8* StringIt = Strings.GetData();
	for (FSerializedNameHeader Header : Headers)
	{
		FNameSerializedView Name(StringIt, Header.Len(), !!Header.IsUtf16());
		StringIt += Name.NumBytes();
		*OutIt++ = BatchLoadNameWithHash(Name, INTEL_ORDER64(*Hashes++));
	}
	check(StringIt == Strings.end());
}

static void LoadSeparatedNameBatchInInputOrder(	TArray<FDisplayNameEntryId>& Out,
												TArrayView<const FSerializedNameHeader> Headers, 
												TArrayView<const uint8> Strings)
{
	check(Out.Num() == Headers.Num());

	TArray<FDisplayNameEntryId>::TIterator OutIt(Out);
	const uint8* StringIt = Strings.GetData();
	for (FSerializedNameHeader Header : Headers)
	{
		FNameSerializedView Name(StringIt, Header.Len(), !!Header.IsUtf16());
		StringIt += Name.NumBytes();
		*OutIt++ = BatchLoadNameWithoutHash(Name);
	}
	check(StringIt == Strings.end());
}

#if WITH_CASE_PRESERVING_NAME

FORCENOINLINE // To help profiling
static void LoadDisplayNames(const TArray<FNameComparisonLoad>& ComparisonLoads)
{
	FNamePool& Pool = GetNamePoolPostInit();

	// Reserve load requests
	TArray<FNameDisplayLoad> LoadShards[FNamePoolShards];
	const uint32 ReservePerShard = (8 * ComparisonLoads.Num()) / (7 * FNamePoolShards);
	for (TArray<FNameDisplayLoad>& LoadShard : LoadShards)
	{
		LoadShard.Reserve(ReservePerShard);
	}

	// Hash display names and prepare load requests
	for (const FNameComparisonLoad& ComparisonLoad : ComparisonLoads)
	{
		FNameDisplayValue DisplayValue(ComparisonLoad.In.Name);
		DisplayValue.ComparisonId = ComparisonLoad.Out->GetLoadedComparisonId();

		bool bReuseEntry = Pool.ReuseComparisonEntry(ComparisonLoad.bOutCreatedNewEntry, DisplayValue);
		LoadShards[DisplayValue.Hash.ShardIndex].Emplace(FNameDisplayLoad {DisplayValue, ComparisonLoad.Out, bReuseEntry});
	}

	// Issue load requests
	uint32 ShardIndex = 0;
	for (TArrayView<FNameDisplayLoad> LoadShard : LoadShards)
	{
		Pool.StoreBatch(ShardIndex++, LoadShard);
	}
}

#endif  // WITH_CASE_PRESERVING_NAME

// Helper struct for loading part of a name batch into a shard
struct FShardTarget
{
	uint32 Num;
	uint32 SortIdx;
};
using FShardTargetArray = FShardTarget[FNamePoolShards];

static void InitializeTargets(FShardTargetArray& Targets, const TArrayView<const uint64> Hashes)
{
	FMemory::Memzero(Targets);

	// Count names / shard
	for (uint64 Hash : Hashes)
	{
		Targets[FNameHash::GetShardIndex(Hash)].Num++;
	}

	// Set starting index per shard
	uint32 SortIdx = 0;
	for (FShardTarget& Target : Targets)
	{
		Target.SortIdx = SortIdx;
		SortIdx += Target.Num;
	}

	check(SortIdx == Hashes.Num());
}


static void LoadInterleavedNameBatchInShardOrder(	TArray<FDisplayNameEntryId>& Out,
													TArrayView<const uint64> Hashes,
													TArrayView<const uint8> HeadersAndStrings)
{
	check(CanCastUtf16ToWideCharWithoutConversion());
	check(Out.Num() == Hashes.Num());

	const uint32 Num = Out.Num();

	FShardTargetArray Targets;
	InitializeTargets(Targets, Hashes);
	
	// Generate name views and count names / shard
	TArray<FNameStringView> Names;
	Names.SetNumUninitialized(Num);
	const uint8* NameIt = HeadersAndStrings.GetData();
	for (uint32 Idx = 0; Idx < Num; ++Idx)
	{
		Names[Idx] = LoadNameHeader(/* in-out */ NameIt).CastToNameView(); 
	}
	check(NameIt == HeadersAndStrings.end());

	// Prepare batch loading requests sorted by shard index
	TArray<FNameComparisonLoad> ShardSortedLoads;
	ShardSortedLoads.SetNumUninitialized(Num);
	for (uint32 Idx = 0; Idx < Num; ++Idx)
	{
		uint64 Hash = INTEL_ORDER64(Hashes[Idx]);
		FShardTarget& Target = Targets[FNameHash::GetShardIndex(Hash)];

		ShardSortedLoads[Target.SortIdx] = FNameComparisonLoad {FNameComparisonValue(Names[Idx], Hash), &Out[Idx]};

		++Target.SortIdx;
	}
	
	// Loop over shard and load batches
	FNamePool& Pool = GetNamePoolPostInit();
	FNameComparisonLoad* LoadIt = ShardSortedLoads.GetData();
	for (FShardTarget& Target : Targets)
	{
		TArrayView<FNameComparisonLoad> Batch(LoadIt, Target.Num);
		LoadIt += Target.Num;
		Pool.StoreBatch(&Target - Targets, Batch);
	}
	check(LoadIt == ShardSortedLoads.GetData() + ShardSortedLoads.Num());

#if WITH_CASE_PRESERVING_NAME
	LoadDisplayNames(ShardSortedLoads);
#endif
}

static void LoadSeparatedNameBatchInShardOrder(	TArray<FDisplayNameEntryId>& Out,
												const uint64* Hashes,
												TArrayView<const FSerializedNameHeader> Headers, 
												TArrayView<const uint8> Strings)											
{
	check(CanCastUtf16ToWideCharWithoutConversion());
	check(Out.Num() == Headers.Num());

	// Number of names and bytes per shard + indices into shard sorted arrays
	struct FExtendedShardTarget
	{
		uint32 NumNames;
		uint32 NumBytes;
		uint32 NameIdx;
		uint32 ByteIdx;
	}
	Targets[FNamePoolShards] = {};

	// Count names and bytes per shard
	for (int32 Idx = 0; Idx < Headers.Num(); ++Idx)
	{
		FExtendedShardTarget& Target = Targets[FNameHash::GetShardIndex(INTEL_ORDER64(Hashes[Idx]))];
		++Target.NumNames;
		Target.NumBytes += Headers[Idx].NumBytes();
	}

	// Find name and byte start offsets per shard
	uint32 ByteIdx = 0;
	uint32 NameIdx = 0;

	for (FExtendedShardTarget& Target : Targets)
	{
		Target.NameIdx = NameIdx;
		Target.ByteIdx = ByteIdx;
		
		NameIdx += Target.NumNames;
		ByteIdx += Target.NumBytes;
	}
	
	const uint32 NumBytes = ByteIdx;
	const uint32 NumNames = NameIdx;
	
	check(NumNames == Headers.Num());
	check(NumBytes == Strings.Num());

	// Sort headers in shard order, scatter read name data in shard order 
	// and save the input order in OutIndices

	TArray<FNameComparisonLoad> ShardSortedLoads;
	TArray<uint8> ShardSortedStrings;
	ShardSortedLoads.SetNumUninitialized(NumNames);
	ShardSortedStrings.SetNumUninitialized(NumBytes);

	const uint8* UnsortedStringsIt = Strings.GetData();
	for (uint32 Idx = 0; Idx < NumNames; ++Idx)
	{
		FSerializedNameHeader Header = Headers[Idx];
		uint64 Hash = INTEL_ORDER64(Hashes[Idx]);

		// Copy name data in read order
		FExtendedShardTarget& Target = Targets[FNameHash::GetShardIndex(Hash)];
		void* NameData = &ShardSortedStrings[Target.ByteIdx];
		const FNameStringView Name(NameData, Header.Len(), !!Header.IsUtf16());

		FMemory::Memcpy(NameData, UnsortedStringsIt, Name.BytesWithoutTerminator());
		UnsortedStringsIt += Name.BytesWithoutTerminator();

		// Prepare batch loading request
		ShardSortedLoads[Target.NameIdx] = FNameComparisonLoad {FNameComparisonValue(Name, Hash), &Out[Idx]};

		++Target.NameIdx;
		Target.ByteIdx += Name.BytesWithoutTerminator();
	}
	check(UnsortedStringsIt == Strings.end());
	
	// Loop over shards and issue batch loads
	FNamePool& Pool = GetNamePoolPostInit();
	FNameComparisonLoad* LoadIt = ShardSortedLoads.GetData();
	for (FExtendedShardTarget& Target : Targets)
	{
		TArrayView<FNameComparisonLoad> Batch(LoadIt, Target.NumNames);
		LoadIt += Target.NumNames;
		Pool.StoreBatch(&Target - Targets, Batch);
	}
	check(LoadIt == ShardSortedLoads.GetData() + ShardSortedLoads.Num());

#if WITH_CASE_PRESERVING_NAME
	LoadDisplayNames(ShardSortedLoads);
#endif
}

void LoadNameBatch(TArray<FDisplayNameEntryId>& OutNames, TArrayView<const uint8> NameData, TArrayView<const uint8> HashData, ENameBatchLoadingFlags Flags)
{
	check(IsAligned(NameData.GetData(), sizeof(uint64)));
	check(IsAligned(HashData.GetData(), sizeof(uint64)));
	check(IsAligned(HashData.Num(), sizeof(uint64)));
	check(HashData.Num() > 0);

	const uint64* HashDataIt = reinterpret_cast<const uint64*>(HashData.GetData());
	uint64 HashVersion = INTEL_ORDER64(HashDataIt[0]);
	const uint32 Num = HashData.Num() / sizeof(uint64) - 1;
	TArrayView<const uint64> Hashes = MakeArrayView(HashDataIt + 1, Num);

	OutNames.SetNumUninitialized(Num);

	if (!CanUseSavedHashes(HashVersion))
	{
		LoadInterleavedNameBatchInInputOrder(OutNames, NameData);
	}
	else if (Num < FNamePoolShards || EnumHasAnyFlags(Flags, ENameBatchLoadingFlags::RespectOrder))
	{
		LoadInterleavedNameBatchInInputOrder(OutNames, Hashes, NameData);
	}
	else
	{
		LoadInterleavedNameBatchInShardOrder(OutNames, Hashes, NameData);
	}
}

struct FNameBatchLoader
{
	TArrayView<const uint64> Hashes;
	TArrayView<const FSerializedNameHeader> Headers;
	TArrayView<const uint8> Strings;
	TArray<uint8> Data;
	ENameBatchLoadingFlags Flags;
	
	FNameBatchLoader(ENameBatchLoadingFlags InFlags)
		: Flags(InFlags)
	{
	}

	// @return true if there's anything to load
	bool Read(FArchive& Ar)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE_STR("FNameBatchLoader::Read");
		uint32 Num = 0;
		Ar << Num;

		if (Num == 0)
		{
			return false;		
		}

		uint32 NumStringBytes = 0;
		Ar << NumStringBytes;

		uint64 HashVersion = 0;
		Ar << HashVersion;
		bool bUseSavedHashes = CanUseSavedHashes(HashVersion);

		// Allocate and load hashes, headers and string data in one go
		uint32 NumHashBytes = sizeof(uint64) * Num;
		uint32 NumHeaderBytes = sizeof(FSerializedNameHeader) * Num;
		Data.SetNumUninitialized(NumHashBytes + NumHeaderBytes + NumStringBytes);
		Ar.Serialize(Data.GetData(), Data.Num());

		TArrayView<const uint64> SavedHashes = TArrayView<const uint64>(reinterpret_cast<const uint64*>(Data.GetData()), Num);
		Hashes = bUseSavedHashes ? SavedHashes : TArrayView<const uint64>();
		Headers = TArrayView<const FSerializedNameHeader>(reinterpret_cast<const FSerializedNameHeader*>(SavedHashes.end()), Num);
		Strings = TArrayView<const uint8>(reinterpret_cast<const uint8*>(Headers.end()), NumStringBytes);

		return !Ar.IsError();
	}

	TArray<FDisplayNameEntryId> Load()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE_STR("FNameBatchLoader::Load");
		check(Headers.Num());

		TArray<FDisplayNameEntryId> Out;
		Out.SetNumUninitialized(Headers.Num());

		if (Hashes.Num() == 0)
		{
			LoadSeparatedNameBatchInInputOrder(Out, Headers, Strings);
		}
		else if (Headers.Num() < FNamePoolShards || EnumHasAnyFlags(Flags, ENameBatchLoadingFlags::RespectOrder))
		{
			LoadSeparatedNameBatchInInputOrder(Out, Hashes.GetData(), Headers, Strings);
		}
		else
		{
			LoadSeparatedNameBatchInShardOrder(Out, Hashes.GetData(), Headers, Strings);
		}

		return Out;
	}
};

TArray<FDisplayNameEntryId> LoadNameBatch(FArchive& Ar, ENameBatchLoadingFlags Flags)
{
	FNameBatchLoader Loader(Flags);

	if (Loader.Read(Ar))
	{
		return Loader.Load();
	}

	return TArray<FDisplayNameEntryId>();
}

struct FNameBatchAsyncLoader : FNameBatchLoader
{
	FNameBatchAsyncLoader(ENameBatchLoadingFlags InFlags)
		: FNameBatchLoader(InFlags)
	{
	}
	
	~FNameBatchAsyncLoader()
	{
		FGenericPlatformProcess::ReturnSynchEventToPool(/* can be null */ DoneEvent);
	}

	bool ShouldLoadAsync(uint32 MaxWorkers) const
	{
		return MaxWorkers && Hashes.Num() >= FNamePoolShards && Hashes.Num() > 30000; //-V590
	}

	void PrepareWork()
	{
		DoneEvent = FPlatformProcess::GetSynchEventFromPool();
		Out.SetNumUninitialized(Headers.Num());
	}

	void DoWork()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE_STR("FNameBatchAsyncLoader::DoWork");
		if (EnumHasAnyFlags(Flags, ENameBatchLoadingFlags::RespectOrder))
		{
			LoadSeparatedNameBatchInInputOrder(Out, Hashes.GetData(), Headers, Strings);
		}
		else
		{
			LoadSeparatedNameBatchInShardOrder(Out, Hashes.GetData(), Headers, Strings);
		}
		DoneEvent->Trigger();
	}

	TArray<FDisplayNameEntryId> GetResult()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE_STR("FNameBatchAsyncLoader::GetResult");
		check(DoneEvent);

		DoneEvent->Wait();
		FPlatformProcess::ReturnSynchEventToPool(DoneEvent);
		DoneEvent = nullptr;

		return MoveTemp(Out);
	}

	FEvent* DoneEvent = nullptr;
	TArray<FDisplayNameEntryId> Out;
};

TFunction<TArray<FDisplayNameEntryId>()> LoadNameBatchAsync(FArchive& Ar, uint32 MaxWorkers, ENameBatchLoadingFlags Flags)
{
	check(MaxWorkers > 0);

	// Ref-count since worker tasks may outlive the returned function
	TSharedPtr<FNameBatchAsyncLoader, ESPMode::ThreadSafe> Loader(new FNameBatchAsyncLoader(Flags));

	if (Loader->Read(Ar))
	{
		if (Loader->ShouldLoadAsync(MaxWorkers))
		{
			Loader->PrepareWork();
			Async(EAsyncExecution::TaskGraph, [=](){ Loader->DoWork(); });
			return [=](){ return Loader->GetResult(); };
		}
		else
		{
			// Sync load
			return [Out = Loader->Load()]() { return Out; };
		}
	}

	return []() { return TArray<FDisplayNameEntryId>(); };
}


#if 0 && ALLOW_NAME_BATCH_SAVING  

FORCENOINLINE void PerfTestLoadNameBatch(TArray<FLoadedNameEntryId>& OutNames, TArrayView<const uint8> NameData, TArrayView<const uint8> HashData)
{
	LoadNameBatch(OutNames, NameData, HashData);
}

static bool WithinBlock(uint8* BlockBegin, const FNameEntry* Entry)
{
	return UPTRINT(Entry) >= UPTRINT(BlockBegin) && UPTRINT(Entry) < UPTRINT(BlockBegin) + FNameEntryAllocator::BlockSizeBytes;
}

#include "GenericPlatform/GenericPlatformFile.h"

static void WriteBlobFile(const TCHAR* FileName, const TArray<uint8>& Blob)
{
	TUniquePtr<IFileHandle> FileHandle(IPlatformFile::GetPlatformPhysical().OpenWrite(FileName));
	FileHandle->Write(Blob.GetData(), Blob.Num());
}

static TArray<uint8> ReadBlobFile(const TCHAR* FileName)
{
	TArray<uint8> Out;
	TUniquePtr<IFileHandle> FileHandle(IPlatformFile::GetPlatformPhysical().OpenRead(FileName));
	if (FileHandle)
	{
		Out.AddUninitialized(FileHandle->Size());
		FileHandle->Read(Out.GetData(), Out.Num());
	}

	return Out;
}

CORE_API int SaveNameBatchTestFiles();
CORE_API int LoadNameBatchTestFiles(bool bArchivePath);

int SaveNameBatchTestFiles()
{
	uint8** Blocks = GetNamePool().GetBlocksForDebugVisualizer();
	uint32 BlockIdx = 0;
	TArray<FNameEntryId> NameEntries;
	for (const FNameEntry* Entry : FName::DebugDump())
	{
		BlockIdx += !WithinBlock(Blocks[BlockIdx], Entry);
		check(WithinBlock(Blocks[BlockIdx], Entry));

		FNameEntryHandle Handle(BlockIdx, (UPTRINT(Entry) - UPTRINT(Blocks[BlockIdx]))/FNameEntryAllocator::Stride);
		NameEntries.Add(Handle);
	}

	TArray<uint8> NameData;
	TArray<uint8> HashData;
	SaveNameBatch(MakeArrayView(NameEntries), NameData, HashData);

	WriteBlobFile(TEXT("TestNameBatch.Names"), NameData);
	WriteBlobFile(TEXT("TestNameBatch.Hashes"), HashData);

	TArray<uint8> ArchData;
	FMemoryWriter ArchWriter(ArchData);
	SaveNameBatch(MakeArrayView(NameEntries), ArchWriter);
	WriteBlobFile(TEXT("TestNameBatch.Archive"), ArchData);

	return NameEntries.Num();
}

int LoadNameBatchTestFiles(bool bArchivePath)
{
	if (bArchivePath)
	{
		TArray<uint8> ArchData = ReadBlobFile(TEXT("TestNameBatch.Archive"));
		FMemoryReader ArchReader(ArchData);
		TArray<FLoadedNameEntryId> NameEntries = LoadNameBatch(ArchiveReader);
		return NameEntries.Num();
	}
	else
	{
		TArray<uint8> NameData = ReadBlobFile(TEXT("TestNameBatch.Names"));
		TArray<uint8> HashData = ReadBlobFile(TEXT("TestNameBatch.Hashes"));

		TArray<FLoadedNameEntryId> NameEntries;
		if (HashData.Num())
		{
			ReserveNameBatch(NameData.Num(), HashData.Num());
			PerfTestLoadNameBatch(NameEntries, MakeArrayView(NameData), MakeArrayView(HashData));
		}

		return NameEntries.Num();
	}
}

#endif

#if WITH_TESTS
static void TestNameBatch()
{
#if ALLOW_NAME_BATCH_SAVING

	TArray<FDisplayNameEntryId> Names;
	TArray<uint8> NameData;
	TArray<uint8> HashData;

	// Test empty batch
	SaveNameBatch(MakeArrayView(Names), NameData, HashData);
	check(NameData.Num() == 0);
	LoadNameBatch(Names, MakeArrayView(NameData), MakeArrayView(HashData));
	check(Names.Num() == 0);

	// Test empty / "None" name and another EName
	Names.Add(FDisplayNameEntryId(FName()));
	Names.Add(FDisplayNameEntryId(FName(NAME_Box)));

	// Test long strings
	FString MaxLengthAnsi;
	MaxLengthAnsi.Reserve(NAME_SIZE);
	while (MaxLengthAnsi.Len() < NAME_SIZE)
	{
		MaxLengthAnsi.Append("0123456789ABCDEF");
	}
	MaxLengthAnsi = MaxLengthAnsi.Left(NAME_SIZE - 1);

	FString MaxLengthWide = MaxLengthAnsi;
	MaxLengthWide[200] = 500;

	for (const FString& MaxLength : {MaxLengthAnsi, MaxLengthWide})
	{
		Names.Add(FDisplayNameEntryId(FName(*MaxLength)));
		Names.Add(FDisplayNameEntryId(FName(*MaxLength + NAME_SIZE - 255)));
		Names.Add(FDisplayNameEntryId(FName(*MaxLength + NAME_SIZE - 256)));
		Names.Add(FDisplayNameEntryId(FName(*MaxLength + NAME_SIZE - 257)));
	}

	// Test UTF-16 alignment
	FString Wide("Wide ");
	Wide[4] = 60000;

	Names.Add(FDisplayNameEntryId(FName(*Wide)));
	Names.Add(FDisplayNameEntryId(FName("odd")));
	Names.Add(FDisplayNameEntryId(FName(*Wide)));
	Names.Add(FDisplayNameEntryId(FName("even")));
	Names.Add(FDisplayNameEntryId(FName(*Wide)));

	// Test preserve different casing
	FName("case");
	Names.Add(FDisplayNameEntryId(FName("CASE")));
	Names.Add(FDisplayNameEntryId(FName("case")));
	Names.Add(FDisplayNameEntryId(FName("casE")));

	// Roundtrip names
	SaveNameBatch(MakeArrayView(Names), NameData, HashData);
	check(NameData.Num() > 0);
	TArray<FDisplayNameEntryId> LoadedNames;
	LoadNameBatch(LoadedNames, MakeArrayView(NameData), MakeArrayView(HashData));
	check(LoadedNames == Names);

	// Test changing hash version
	HashData[0] = 0xba;
	HashData[1] = 0xad;
	LoadNameBatch(LoadedNames, MakeArrayView(NameData), MakeArrayView(HashData));
	check(LoadedNames == Names);

	// Test determinism
	TArray<uint8> NameData2;
	TArray<uint8> HashData2;

	auto ClearAndReserveMemoryWithBytePattern = 
		[](TArray<uint8>& Out, uint8 Pattern, uint32 Num)
	{
		Out.Init(Pattern, Num);
		Out.Empty(Out.Max());
	};
	
	ClearAndReserveMemoryWithBytePattern(NameData2, 0xaa, NameData.Num());
	ClearAndReserveMemoryWithBytePattern(HashData2, 0xaa, HashData.Num());
	ClearAndReserveMemoryWithBytePattern(NameData,	0xbb, NameData.Num());
	ClearAndReserveMemoryWithBytePattern(HashData,	0xbb, HashData.Num());
	
	SaveNameBatch(MakeArrayView(Names), NameData, HashData);
	SaveNameBatch(MakeArrayView(Names), NameData2, HashData2);
	
	check(NameData == NameData2);
	check(HashData == HashData2);

	// Test FArchive-based Load/SaveNameBatch()

	auto TestArchiveRoundtrip = [](const TArray<FDisplayNameEntryId>& Names)
	{
		TArray<uint8> ArchiveData;
		FMemoryWriter ArchiveWriter(ArchiveData);
		SaveNameBatch(MakeArrayView(Names), ArchiveWriter);

		FMemoryReader ArchiveReader(ArchiveData);
		TArray<FDisplayNameEntryId> ArchiveRoundtrippedNames = LoadNameBatch(ArchiveReader);
		
		check(ArchiveRoundtrippedNames == Names);
	};

	TestArchiveRoundtrip(Names);
	
	TArray<FDisplayNameEntryId> LargeBatch = Names;
	for (char C1 = 'A'; C1 <= 'z' ; ++C1)
	{
		for (char C2 = 'A'; C2 < 'z'; ++C2)
		{
			char Str[]  = {C1, C2, '\0'};

			// Use display index to get some duplicate comparison ids like "aa" and "aA"
			LargeBatch.Add(FDisplayNameEntryId(FName(Str)));
		}
	}
	check(LargeBatch.Num() > FNamePoolShards);
	TestArchiveRoundtrip(LargeBatch);

#endif // ALLOW_NAME_BATCH_SAVING
}
#endif // WITH_TESTS

#if !UE_BUILD_SHIPPING && !UE_BUILD_TEST

#include "Containers/StackTracker.h"
static TAutoConsoleVariable<int32> CVarLogGameThreadFNameChurn(
	TEXT("LogGameThreadFNameChurn.Enable"),
	0,
	TEXT("If > 0, then collect sample game thread fname create, periodically print a report of the worst offenders."));

static TAutoConsoleVariable<int32> CVarLogGameThreadFNameChurn_PrintFrequency(
	TEXT("LogGameThreadFNameChurn.PrintFrequency"),
	300,
	TEXT("Number of frames between churn reports."));

static TAutoConsoleVariable<int32> CVarLogGameThreadFNameChurn_Threshhold(
	TEXT("LogGameThreadFNameChurn.Threshhold"),
	10,
	TEXT("Minimum average number of fname creations per frame to include in the report."));

static TAutoConsoleVariable<int32> CVarLogGameThreadFNameChurn_SampleFrequency(
	TEXT("LogGameThreadFNameChurn.SampleFrequency"),
	1,
	TEXT("Number of fname creates per sample. This is used to prevent churn sampling from slowing the game down too much."));

static TAutoConsoleVariable<int32> CVarLogGameThreadFNameChurn_StackIgnore(
	TEXT("LogGameThreadFNameChurn.StackIgnore"),
	4,
	TEXT("Number of items to discard from the top of a stack frame."));

static TAutoConsoleVariable<int32> CVarLogGameThreadFNameChurn_RemoveAliases(
	TEXT("LogGameThreadFNameChurn.RemoveAliases"),
	1,
	TEXT("If > 0 then remove aliases from the counting process. This essentialy merges addresses that have the same human readable string. It is slower."));

static TAutoConsoleVariable<int32> CVarLogGameThreadFNameChurn_StackLen(
	TEXT("LogGameThreadFNameChurn.StackLen"),
	3,
	TEXT("Maximum number of stack frame items to keep. This improves aggregation because calls that originate from multiple places but end up in the same place will be accounted together."));


struct FSampleFNameChurn
{
	FStackTracker GGameThreadFNameChurnTracker;
	bool bEnabled;
	int32 CountDown;
	uint64 DumpFrame;

	FSampleFNameChurn()
		: bEnabled(false)
		, CountDown(MAX_int32)
		, DumpFrame(0)
	{
	}

	void NameCreationHook()
	{
		bool bNewEnabled = CVarLogGameThreadFNameChurn.GetValueOnGameThread() > 0;
		if (bNewEnabled != bEnabled)
		{
			check(IsInGameThread());
			bEnabled = bNewEnabled;
			if (bEnabled)
			{
				CountDown = CVarLogGameThreadFNameChurn_SampleFrequency.GetValueOnGameThread();
				DumpFrame = GFrameCounter + CVarLogGameThreadFNameChurn_PrintFrequency.GetValueOnGameThread();
				GGameThreadFNameChurnTracker.ResetTracking();
				GGameThreadFNameChurnTracker.ToggleTracking(true, true);
			}
			else
			{
				GGameThreadFNameChurnTracker.ToggleTracking(false, true);
				DumpFrame = 0;
				GGameThreadFNameChurnTracker.ResetTracking();
			}
		}
		else if (bEnabled)
		{
			check(IsInGameThread());
			check(DumpFrame);
			if (--CountDown <= 0)
			{
				CountDown = CVarLogGameThreadFNameChurn_SampleFrequency.GetValueOnGameThread();
				CollectSample();
				if (GFrameCounter > DumpFrame)
				{
					PrintResultsAndReset();
				}
			}
		}
	}

	void CollectSample()
	{
		check(IsInGameThread());
		GGameThreadFNameChurnTracker.CaptureStackTrace(CVarLogGameThreadFNameChurn_StackIgnore.GetValueOnGameThread(), nullptr, CVarLogGameThreadFNameChurn_StackLen.GetValueOnGameThread(), CVarLogGameThreadFNameChurn_RemoveAliases.GetValueOnGameThread() > 0);
	}
	void PrintResultsAndReset()
	{
		DumpFrame = GFrameCounter + CVarLogGameThreadFNameChurn_PrintFrequency.GetValueOnGameThread();
		FOutputDeviceRedirector* Log = FOutputDeviceRedirector::Get();
		float SampleAndFrameCorrection = float(CVarLogGameThreadFNameChurn_SampleFrequency.GetValueOnGameThread()) / float(CVarLogGameThreadFNameChurn_PrintFrequency.GetValueOnGameThread());
		GGameThreadFNameChurnTracker.DumpStackTraces(CVarLogGameThreadFNameChurn_Threshhold.GetValueOnGameThread(), *Log, SampleAndFrameCorrection);
		GGameThreadFNameChurnTracker.ResetTracking();
	}
};

FSampleFNameChurn GGameThreadFNameChurnTracker;

void CallNameCreationHook()
{
	if (GIsRunning && IsInGameThread())
	{
		GGameThreadFNameChurnTracker.NameCreationHook();
	}
}

#endif

// Make Clang keep FNameDebugVisualizer type debug info needed for debug visualizers
FNameDebugVisualizer::FNameDebugVisualizer(FClangKeepDebugInfo) {}

uint8** FNameDebugVisualizer::GetBlocks()
{
	static_assert(EntryStride == FNameEntryAllocator::Stride,	"Natvis constants out of sync with actual constants");
	static_assert(BlockBits == FNameMaxBlockBits,				"Natvis constants out of sync with actual constants");
	static_assert(OffsetBits == FNameBlockOffsetBits,			"Natvis constants out of sync with actual constants");

	return ((FNamePool*)(NamePoolData))->GetBlocksForDebugVisualizer();
}

FString FScriptName::ToString() const
{
	return ScriptNameToName(*this).ToString();
}

FString FMemoryImageName::ToString() const
{
	return FName(*this).ToString();
}

uint32 Freeze::IntrinsicAppendHash(const FMemoryImageName* DummyObject, const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams, FSHA1& Hasher)
{
	const uint32 SizeFromFields = LayoutParams.WithCasePreservingFName() ? sizeof(TMemoryImageNameLayout<1>) : sizeof(TMemoryImageNameLayout<0>);
	return Freeze::AppendHashForNameAndSize(TypeDesc.Name, SizeFromFields, Hasher);
}

void Freeze::ApplyMemoryImageNamePatch(void* NameDst, const FMemoryImageName& Name, const FPlatformTypeLayoutParameters& LayoutParams)
{
	if (!LayoutParams.WithCasePreservingFName())
	{
		TMemoryImageNameLayout<0>* ToWrite = new(NameDst) TMemoryImageNameLayout<0>();
		ToWrite->ComparisonIndex = Name.ComparisonIndex;
#if !UE_FNAME_OUTLINE_NUMBER
		ToWrite->NumberOrDummy = Name.Number;
#endif
	}
	else
	{
		TMemoryImageNameLayout<1>* ToWrite = new(NameDst) TMemoryImageNameLayout<1>();
		ToWrite->ComparisonIndex = Name.ComparisonIndex;
#if !UE_FNAME_OUTLINE_NUMBER
		ToWrite->NumberOrDummy = Name.Number;
#endif
#if WITH_CASE_PRESERVING_NAME
		ToWrite->DisplayIndex = Name.DisplayIndex;
#else
		ToWrite->DisplayIndex = Name.ComparisonIndex;
#endif
	}
}

void Freeze::IntrinsicWriteMemoryImage(FMemoryImageWriter& Writer, const FMemoryImageName& Object, const FTypeLayoutDesc& LayoutDesc)
{
	const FPlatformTypeLayoutParameters& TargetLayoutParameters = Writer.GetTargetLayoutParams();
	if (TargetLayoutParameters.IsCurrentPlatform())
	{
		Writer.WriteFMemoryImageName(sizeof(FMemoryImageName), FName(Object));
	}
	else
	{
		ensureMsgf(WITH_CASE_PRESERVING_NAME, TEXT("Cooking memory images containing names when WITH_CASE_PRESERVING_NAME is false can lead to cook determinism issues with string casing."));
		if (!TargetLayoutParameters.WithCasePreservingFName())
		{
			Writer.WriteFMemoryImageName(sizeof(TMemoryImageNameLayout<0>), FName(Object));
		}
		else
		{
			Writer.WriteFMemoryImageName(sizeof(TMemoryImageNameLayout<1>), FName(Object));
		}
	}
}

uint32 Freeze::IntrinsicUnfrozenCopy(const FMemoryUnfreezeContent& Context, const FMemoryImageName& Object, void* OutDst)
{
	const FPlatformTypeLayoutParameters& TargetLayoutParameters = Context.FrozenLayoutParameters;
	if (TargetLayoutParameters.IsCurrentPlatform())
	{
		new(OutDst) FMemoryImageName(Object);
		return sizeof(FMemoryImageName);
	}
	else
	{
		if (!TargetLayoutParameters.WithCasePreservingFName())
		{
			const TMemoryImageNameLayout<0>& Loaded = reinterpret_cast<const TMemoryImageNameLayout<0>&>(Object);
			FMemoryImageName* Dest = new(OutDst) FMemoryImageName();
			Dest->ComparisonIndex = Loaded.ComparisonIndex;
#if !UE_FNAME_OUTLINE_NUMBER
			Dest->Number = Loaded.NumberOrDummy;
#endif
#if WITH_CASE_PRESERVING_NAME
			Dest->DisplayIndex = Loaded.ComparisonIndex;
#endif

			return sizeof(Loaded);
		}
		else
		{
			const TMemoryImageNameLayout<1>& Loaded = reinterpret_cast<const TMemoryImageNameLayout<1>&>(Object);
			FMemoryImageName* Dest = new(OutDst) FMemoryImageName();
			Dest->ComparisonIndex = Loaded.ComparisonIndex;
#if !UE_FNAME_OUTLINE_NUMBER
			Dest->Number = Loaded.NumberOrDummy;
#endif
#if WITH_CASE_PRESERVING_NAME
			Dest->DisplayIndex = Loaded.DisplayIndex;
#endif

			return sizeof(Loaded);
		}
	}
}
void Freeze::IntrinsicWriteMemoryImage(FMemoryImageWriter& Writer, const FScriptName& Object, const FTypeLayoutDesc&)
{
	Writer.WriteFScriptName(Object);
}

bool ShouldReplicateAsInteger(EName Ename, const FName& Name)
{
	return Ename <= EName(MAX_NETWORKED_HARDCODED_NAME) && Name.GetNumber() == NAME_NO_NUMBER_INTERNAL;
}

PRAGMA_RESTORE_UNSAFE_TYPECAST_WARNINGS

// Console command implementations
namespace UE::Name::Private
{
	// Used instead of FOutputDeviceFile because that output device is disabled by some preprocessor definitions.
	struct FOutputDeviceWrapper : public FOutputDevice
	{
		FOutputDeviceWrapper(FArchive& InArchive)
			: Ar(InArchive)
		{
			bSuppressEventTag = true;
		}

		virtual void Serialize(const TCHAR* Data, ELogVerbosity::Type Verbosity, const FName& Category)
		{
			FOutputDeviceHelper::FormatCastAndSerializeLine(Ar, Data, Verbosity, Category, FPlatformTime::Seconds(), bSuppressEventTag, bAutoEmitLineTerminator);
		}

	private:
		FArchive& Ar;
	};

	void DebugDumpInternal(bool bNumbered, int32 Num, FOutputDevice& Out)
	{
		const TArray<const FNameEntry*> Entries = FName::DebugDump();
		TConstArrayView<const FNameEntry*> IterEntries = Entries;

		if (Num < MAX_int32)
		{
			IterEntries.RightInline(Num);
		}

		for (const FNameEntry* Entry : IterEntries)
		{
#if UE_FNAME_OUTLINE_NUMBER
			if (Entry->IsNumbered() == bNumbered)
#endif
			{
				Entry->DebugDump(Out);
			}
		}
	}

	// List FNames to the output device 
	void ListFNames(const TArray<FString>& Args, UWorld*, FOutputDevice& Out)
	{
		int32 Num = MAX_int32;
		for (const FString& Arg : Args)
		{
			FParse::Value(*Arg, TEXT("num="), Num);
		}

		DebugDumpInternal(false, Num, Out);
	}

	// Dump FNames to a file
	void DumpFNames(const TArray<FString>& Args, UWorld*, FOutputDevice& Out)
	{
		int32 Num = MAX_int32;
		for (const FString& Arg : Args)
		{
			FParse::Value(*Arg, TEXT("num="), Num);
		}

		FString Filename = FPaths::ProfilingDir() + FString::Printf(TEXT("FNames_(%s).txt"), *FDateTime::Now().ToString(TEXT("%Y%m%d_%H%M%S")));
		TUniquePtr<FArchive> FileAr{IFileManager::Get().CreateFileWriter(*Filename)};
		if (!FileAr)
		{
			Out.Logf(ELogVerbosity::Error, TEXT("Failed to open outfile file to dump FNames."));
			return;
		}

		FOutputDeviceWrapper OutDevice(*FileAr);
		DebugDumpInternal(false, Num, OutDevice);
		Out.Logf(TEXT("Wrote FNames to %s"), *Filename);
	}

	// List FNames to the output device 
	void ListNumberedFNames(const TArray<FString>& Args, UWorld*, FOutputDevice& Out)
	{
#if UE_FNAME_OUTLINE_NUMBER
		int32 Num = MAX_int32;
		for (const FString& Arg : Args)
		{
			FParse::Value(*Arg, TEXT("num="), Num);
		}

		DebugDumpInternal(true, Num, Out);
#else
		Out.Logf(ELogVerbosity::Error, TEXT("Cannot list numbered FNames as UE_FNAME_OUTLINE_NUMBER is not set so numbers are stored in FName instances rather than the name table."));
#endif
	}

	// Dump FNames to a file
	void DumpNumberedFNames(const TArray<FString>& Args, UWorld*, FOutputDevice& Out)
	{
#if UE_FNAME_OUTLINE_NUMBER
		int32 Num = MAX_int32;
		for (const FString& Arg : Args)
		{
			FParse::Value(*Arg, TEXT("num="), Num);
		}

		FString Filename = FPaths::ProfilingDir() + FString::Printf(TEXT("FNamesNumbered_(%s).txt"), *FDateTime::Now().ToString(TEXT("%Y%m%d_%H%M%S")));
		TUniquePtr<FArchive> FileAr{ IFileManager::Get().CreateFileWriter(*Filename) };
		if (!FileAr)
		{
			Out.Logf(ELogVerbosity::Error, TEXT("Failed to open outfile file to dump FNames."));
			return;
		}

		FOutputDeviceWrapper OutDevice(*FileAr);
		DebugDumpInternal(true, Num, OutDevice);
		Out.Logf(TEXT("Wrote FNames to %s"), *Filename);
#else
		Out.Logf(ELogVerbosity::Error, TEXT("Cannot dump numbered FNames as UE_FNAME_OUTLINE_NUMBER is not set so numbers are stored in FName instances rather than the name table."));
#endif
	}

	// Write FName stats to the output device 
	void GetFNameStats(FOutputDevice& Out)
	{
		FName::DisplayHash(Out);
	}

	void DumpHashCsv(FOutputDevice& Out)
	{
		FString Filename = FPaths::ProfilingDir() + FString::Printf(TEXT("FNameHash_(%s).csv"), *FDateTime::Now().ToString(TEXT("%Y%m%d_%H%M%S")));
		TUniquePtr<FArchive> FileAr{ IFileManager::Get().CreateFileWriter(*Filename) };
		if (!FileAr)
		{
			Out.Logf(ELogVerbosity::Error, TEXT("Failed to open outfile file to dump FName hash stats."));
			return;
		}

		FOutputDeviceWrapper OutDevice(*FileAr);
		GetNamePoolPostInit().LogHashStats(OutDevice);
		Out.Logf(TEXT("Wrote FName hash stats to %s"), *Filename);
	}
}
