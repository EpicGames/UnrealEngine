// Copyright Epic Games, Inc. All Rights Reserved.

#include "Unix/UnixErrorOutputDevice.h"
#include "Containers/StringConv.h"
#include "Logging/LogMacros.h"
#include "CoreGlobals.h"
#include "Logging/StructuredLog.h"
#include "Misc/Parse.h"
#include "Misc/CommandLine.h"
#include "Misc/Paths.h"
#include "Misc/OutputDeviceHelper.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Misc/CoreDelegates.h"
#include "Misc/App.h"
#include "HAL/ExceptionHandling.h"

extern CORE_API bool GIsGPUCrashed;

FUnixErrorOutputDevice::FUnixErrorOutputDevice()
:	ErrorPos(0)
{
}

void FUnixErrorOutputDevice::Serialize(const TCHAR* Msg, ELogVerbosity::Type Verbosity, const class FName& Category)
{
	UE_DEBUG_BREAK();

	if (!GIsCriticalError)
	{
		// First appError.
		GIsCriticalError = 1;
		TCHAR ErrorBuffer[1024];
		ErrorBuffer[0] = 0;
		// pop up a crash window if we are not in unattended mode; 
		if (WITH_EDITOR && FApp::IsUnattended() == false)
		{
			// @TODO Not implemented
			UE_LOGFMT(LogCore, Error, "appError called: {Msg}", Msg );
		}
		// log the warnings to the log
		else
		{
			UE_LOGFMT(LogCore, Error, "appError called: {Msg}", Msg );
		}

		// CheckVerifyFailedImpl writes GErrorHist including a callstack and then calls GError->Logf with only the
		// assertion expression and description. Keep GErrorHist intact if it begins with Msg.
		if (FCString::Strncmp(GErrorHist, Msg, FMath::Min<int32>(UE_ARRAY_COUNT(GErrorHist), FCString::Strlen(Msg))))
		{
			FCString::Strncpy(GErrorHist, Msg, UE_ARRAY_COUNT(GErrorHist) - 5);
			FCString::StrncatTruncateDest(GErrorHist, UE_ARRAY_COUNT(GErrorHist), TEXT("\r\n\r\n"));
		}
		ErrorPos = FCString::Strlen(GErrorHist);
	}
	else
	{
		UE_LOGF(LogCore, Error, "Error reentered: %ls", Msg );
	}

	if( GIsGuarded )
	{
#if PLATFORM_EXCEPTIONS_DISABLED
		UE_DEBUG_BREAK();
#endif
		void* ErrorProgramCounter = GetErrorProgramCounter();
		if (GIsGPUCrashed)
		{
			ReportGPUCrash(Msg, ErrorProgramCounter);
		}
		else
		{
			ReportAssert(Msg, ErrorProgramCounter);
		}
	}
	else
	{
		// We crashed outside the guarded code (e.g. appExit).
		HandleError();
		FPlatformMisc::RequestExit(true, TEXT("FUnixErrorOutputDevice.Serialize.!GIsGuarded"));
	}
}

void FUnixErrorOutputDevice::HandleError()
{
	// make sure we don't report errors twice
	static int32 CallCount = 0;
	int32 NewCallCount = FPlatformAtomics::InterlockedIncrement(&CallCount);
	if (NewCallCount != 1)
	{
		UE_LOGF(LogCore, Error, "HandleError re-entered." );
		return;
	}

	// Trigger the OnSystemFailure hook if it exists
	FCoreDelegates::OnHandleSystemError.Broadcast();

#if !PLATFORM_EXCEPTIONS_DISABLED
	try
	{
#endif // !PLATFORM_EXCEPTIONS_DISABLED
		GIsGuarded = 0;
		GIsRunning = 0;
		GIsCriticalError = 1;
		GLogConsole = NULL;
		GErrorHist[UE_ARRAY_COUNT(GErrorHist)-1] = 0;

		// Dump the error and flush the log.
		UE_LOG(LogCore, Log, TEXT("=== Critical error: ===") LINE_TERMINATOR TEXT("%s") LINE_TERMINATOR, GErrorExceptionDescription);
		UE_LOGF(LogCore, Log, "%ls", GErrorHist);

		GLog->Panic();

		HandleErrorRestoreUI();

		FPlatformMisc::SubmitErrorReport(GErrorHist, EErrorReportMode::Interactive);
		FCoreDelegates::OnShutdownAfterError.Broadcast();
#if !PLATFORM_EXCEPTIONS_DISABLED
	}
	catch(...)
	{}
#endif // !PLATFORM_EXCEPTIONS_DISABLED
}

void FUnixErrorOutputDevice::HandleErrorRestoreUI()
{
}

