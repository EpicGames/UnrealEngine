// Copyright Epic Games, Inc. All Rights Reserved.

#include "Unix/UnixForkPageProtector.h"

#if COMPILE_FORK_PAGE_PROTECTOR
#include "CoreGlobals.h"
#include "HAL/PlatformStackWalk.h"
#include "Hash/CityHash.h"
#include "Misc/Fork.h"
#include "Misc/Paths.h"
#include "Misc/ScopeExit.h"
#include "Misc/ScopeLock.h"
#include "Unix/UnixPlatformFile.h"
#include "Unix/UnixPlatformAtomics.h"

#include <fcntl.h>
#include <pthread.h>
#include <sys/types.h>
#include <sys/stat.h>

// PlatformCrashHandler is not a public symbol but we are in the same module, so just grab the entry point
extern void PlatformCrashHandler(int32 Signal, siginfo_t* Info, void* Context);

namespace UE
{
namespace
{
	const SIZE_T DefaultBlockSize  = 1024U * 1024U * 64U; // 64MB
	const int MinimalFreeBlockSize = 64;
	const int MaxAlignment         = 16;

	struct FAllocationHeader
	{
		void* ActualPtr       = nullptr;
		SIZE_T AllocationSize = 0U;
	};

	struct FFreeNode
	{
		FFreeNode* Next = nullptr;
		SIZE_T FreeSize = 0U;
	};

	struct FBlock
	{
		FFreeNode* FreeList = nullptr;
		FBlock* Next        = nullptr;
		SIZE_T BlockSize    = 0U;
	};

	// The extra size we need per allocation to store Allocation info at to be Aligned
	const int PtrInfoSize = sizeof(FAllocationHeader) + MaxAlignment;

	template <typename T>
	T* AddByteOffsetToPointer(T* Pointer, SIZE_T Size)
	{
		return reinterpret_cast<T*>(reinterpret_cast<uint8_t*>(Pointer) + Size);
	}

	FFreeNode* CreateFreeList(void* Ptr, SIZE_T FreeSize)
	{
		FFreeNode* FreeList = reinterpret_cast<FFreeNode*>(Ptr);

		FreeList->Next     = nullptr;
		FreeList->FreeSize = FreeSize;

		return FreeList;
	}

	FORCENOINLINE FBlock* CreateBlock(SIZE_T Size = DefaultBlockSize)
	{
		// Grab enough extra memory to fit a size(FBlock), though requires remembering our actual size is now BlockSize + sizeof(FBlock)
		FBlock* NewBlock = reinterpret_cast<FBlock*>(
			mmap(nullptr, Size + sizeof(FBlock), PROT_READ | PROT_WRITE, MAP_PRIVATE | MAP_ANON, -1, 0)
		);

		if (NewBlock == nullptr)
		{
			return nullptr;
		}

		NewBlock->FreeList  = CreateFreeList(AddByteOffsetToPointer(NewBlock, sizeof(FBlock)), Size);
		NewBlock->Next      = nullptr;
		NewBlock->BlockSize = Size;

		return NewBlock;
	}

	void* AttemptToAllocateFromBlock(FBlock* CurrentBlock, SIZE_T Size, uint32 Alignment)
	{
		SIZE_T ActualSize = Size + sizeof(FAllocationHeader) + MaxAlignment;

		FFreeNode* CurrentFreeNode  = CurrentBlock->FreeList;
		FFreeNode* PreviousFreeNode = CurrentFreeNode;
		while (CurrentFreeNode != nullptr)
		{
			if (ActualSize < CurrentFreeNode->FreeSize)
			{
				SIZE_T FreeBlockSizeLeft = CurrentFreeNode->FreeSize - ActualSize;

				// 1) If we have enough free size left, link the previous free node to the new free block
				if (FreeBlockSizeLeft > MinimalFreeBlockSize)
				{
					if (PreviousFreeNode == CurrentFreeNode)
					{
						FFreeNode* NextFreeNode = CurrentFreeNode->Next;
						CurrentBlock->FreeList  = CreateFreeList(AddByteOffsetToPointer(CurrentFreeNode, ActualSize), FreeBlockSizeLeft);

						CurrentBlock->FreeList->Next = NextFreeNode;
					}
					else
					{
						PreviousFreeNode->Next = CreateFreeList(AddByteOffsetToPointer(CurrentFreeNode, ActualSize), FreeBlockSizeLeft);
						PreviousFreeNode->Next->Next = CurrentFreeNode->Next;
					}
				}
				// 2) We have no free size left in the free block, add extra reminaing bytes to the alloaction to be re-claimed on a free
				else
				{
					if (PreviousFreeNode == CurrentFreeNode)
					{
						// If our starting FreeList no longer has enough room make the FreeList start the next Node, or nullptr if none exist
						CurrentBlock->FreeList = CurrentFreeNode->Next;
					}
					else
					{
						PreviousFreeNode->Next = CurrentFreeNode->Next;
					}

					// attach extra size to the allocation size, which will be later reclaimined on a free
					Size += FreeBlockSizeLeft;
				}

				void* Result = Align(CurrentFreeNode, Alignment);

				FAllocationHeader* Header = reinterpret_cast<FAllocationHeader*>(Result);

				Header->ActualPtr      = CurrentFreeNode;
				Header->AllocationSize = Size;

				Result = AddByteOffsetToPointer(Result, sizeof(FAllocationHeader));

				return Result;
			}

			PreviousFreeNode = CurrentFreeNode;
			CurrentFreeNode  = CurrentFreeNode->Next;
		}

		return nullptr;
	}
}

FMallocLinked::FMallocLinked(FMalloc* InPreviousMalloc)
	: PreviousMalloc(InPreviousMalloc)
{
}

FMallocLinked::~FMallocLinked()
{
	FBlock* Current = Blocks;
	while (Current != nullptr)
	{
		FBlock* BlockToRemove = Current;
		Current = Current->Next;

		munmap(BlockToRemove, BlockToRemove->BlockSize + sizeof(FBlock));
	}
}

void* FMallocLinked::Malloc(SIZE_T Size, uint32 Alignment)
{
	FScopeLock Lock(&AllocatorMutex);

	if (Size == 0)
	{
		return nullptr;
	}

	if (Alignment > MaxAlignment)
	{
		printf("Alignment > %i not supported\n", MaxAlignment);
		Alignment = MaxAlignment;
	}

	if (Alignment == 0)
	{
		Alignment = MaxAlignment;
	}

	if (Blocks == nullptr)
	{
		// If our Size + extra info required will end be greater then the default size lets carve a perfect block for the alloaction
		if (Size + PtrInfoSize > DefaultBlockSize)
		{
			Blocks = CreateBlock(Size + PtrInfoSize);
		}
		else
		{
			Blocks = CreateBlock();
		}

		if (Blocks == nullptr)
		{
			return nullptr;
		}
	}

	FBlock* CurrentBlock = Blocks;
	while (CurrentBlock != nullptr)
	{
		void* Result = AttemptToAllocateFromBlock(CurrentBlock, Size, Alignment);
		if (Result != nullptr)
		{
			return Result;
		}

		CurrentBlock = CurrentBlock->Next;
	}

	// Insert the next new Block at the head of the list
	CurrentBlock = Blocks;

	// If our Size + extra info required will end be greater then the default size lets carve a perfect block for the alloaction
	if (Size + PtrInfoSize > DefaultBlockSize)
	{
		Blocks = CreateBlock(Size + PtrInfoSize);
	}
	else
	{
		Blocks = CreateBlock();
	}

	if (Blocks == nullptr)
	{
		return nullptr;
	}

	Blocks->Next = CurrentBlock;

	return AttemptToAllocateFromBlock(Blocks, Size, Alignment);
}

void* FMallocLinked::Realloc(void* Ptr, SIZE_T NewSize, uint32 Alignment)
{
	if (Ptr == nullptr)
	{
		return Malloc(NewSize, MaxAlignment);
	}

	SIZE_T AllocationSize = 0U;
	{
		FScopeLock Lock(&AllocatorMutex);

		if (OwnsPointer(Ptr))
		{
			FAllocationHeader* Header = reinterpret_cast<FAllocationHeader*>(reinterpret_cast<uint8*>(Ptr) - sizeof(FAllocationHeader));

			AllocationSize = Header->AllocationSize;
		}
		else if (PreviousMalloc)
		{
			PreviousMalloc->GetAllocationSize(Ptr, AllocationSize);
		}
	}

	void* Result = Malloc(NewSize, MaxAlignment);

	if (Result != nullptr)
	{
		FMemory::Memcpy(Result, Ptr, FMath::Min(NewSize, AllocationSize));
	}

	Free(Ptr);

	return Result;
}

void FMallocLinked::Free(void* Ptr)
{
	FScopeLock Lock(&AllocatorMutex);

	if (Ptr == nullptr)
	{
		return;
	}

	if (PreviousMalloc && !OwnsPointer(Ptr))
	{
		PreviousMalloc->Free(Ptr);
		return;
	}

	bool bPtrFreed = false;

	FBlock* PreviousBlock = Blocks;
	FBlock* CurrentBlock  = Blocks;
	while (CurrentBlock != nullptr)
	{
		if (Ptr >= CurrentBlock && Ptr <= AddByteOffsetToPointer(CurrentBlock, CurrentBlock->BlockSize))
		{
			FAllocationHeader* Header = reinterpret_cast<FAllocationHeader*>(reinterpret_cast<uint8*>(Ptr) - sizeof(FAllocationHeader));

			void* ActualPtr   = Header->ActualPtr;
			SIZE_T ActualSize = Header->AllocationSize + PtrInfoSize;

			// Our block is full lets, just create a new free list and return
			if (CurrentBlock->FreeList == nullptr)
			{
				CurrentBlock->FreeList = CreateFreeList(ActualPtr, ActualSize);
				return;
			}

			FFreeNode* CurrentFreeNode  = CurrentBlock->FreeList;
			FFreeNode* PreviousFreeNode = CurrentFreeNode;
			while (CurrentFreeNode != nullptr)
			{
				// Find where to insert the ptr inbetween two free nodes, or
				//   will become the start of the free list, or
				//   will become the end of the free list

				// 1) Pointer is left most alloaction, make new free list start and check if right merge is possible
				if (ActualPtr < CurrentBlock->FreeList)
				{
					CurrentBlock->FreeList = CreateFreeList(ActualPtr, ActualSize);

					// right merge
					if (AddByteOffsetToPointer(ActualPtr, ActualSize) == CurrentFreeNode)
					{
						CurrentBlock->FreeList->FreeSize += CurrentFreeNode->FreeSize;
						CurrentBlock->FreeList->Next = CurrentFreeNode->Next;
					}
					// no merge
					else
					{
						CurrentBlock->FreeList->Next = CurrentFreeNode;
					}

					bPtrFreed = true;
					break;
				}
				// 2) Pointer is right most allocation, check if can merge to the left free node else new free node
				else if (ActualPtr > CurrentFreeNode && CurrentFreeNode->Next == nullptr)
				{
					// left merge
					if (AddByteOffsetToPointer(CurrentFreeNode, CurrentFreeNode->FreeSize) == ActualPtr)
					{
						CurrentFreeNode->FreeSize += ActualSize;
					}
					// no merge
					else
					{
						CurrentFreeNode->Next = CreateFreeList(ActualPtr, ActualSize);
					}

					bPtrFreed = true;
					break;
				}
				// 3) Previous is Left free node and Current is Right free node. Check if we can merge to the left
				//     else create new free node and check if we can merge Right free node
				else if (ActualPtr > PreviousFreeNode && ActualPtr < CurrentFreeNode)
				{
					// left merge
					if (AddByteOffsetToPointer(PreviousFreeNode, PreviousFreeNode->FreeSize) == ActualPtr)
					{
						PreviousFreeNode->FreeSize += ActualSize;

						// left + right merge
						if (AddByteOffsetToPointer(ActualPtr, ActualSize) == CurrentFreeNode)
						{
							PreviousFreeNode->FreeSize += CurrentFreeNode->FreeSize;
							PreviousFreeNode->Next = CurrentFreeNode->Next;
						}
					}
					// create new free list and check if we can merge right
					else
					{
						PreviousFreeNode->Next = CreateFreeList(ActualPtr, ActualSize);

						// right merge
						if (AddByteOffsetToPointer(ActualPtr, ActualSize) == CurrentFreeNode)
						{
							PreviousFreeNode->Next->FreeSize += CurrentFreeNode->FreeSize;
							PreviousFreeNode->Next->Next = CurrentFreeNode->Next;
						}
						// no merge
						else
						{
							PreviousFreeNode->Next->Next = CurrentFreeNode;
						}
					}

					bPtrFreed = true;
					break;
				}

				PreviousFreeNode = CurrentFreeNode;
				CurrentFreeNode  = CurrentFreeNode->Next;
			}
		}

		if (bPtrFreed)
		{
			return;
		}

		PreviousBlock = CurrentBlock;
		CurrentBlock  = CurrentBlock->Next;
	}

	// Failed to free
	// UE_DEBUG_BREAK()
}

void FMallocLinked::DebugVisualize()
{
	fprintf(stderr, "\nPrinting Allocator layout:\n");

	FBlock* CurrentBlock = Blocks;
	while (CurrentBlock != nullptr)
	{
		SIZE_T BlockSize = CurrentBlock->BlockSize;
		fprintf(stderr, " Block[0x%016llx] BlockSize: %lu\n", reinterpret_cast<uint64>(CurrentBlock), BlockSize);

		FFreeNode* CurrentFreeNode = CurrentBlock->FreeList;
		while (CurrentFreeNode != nullptr)
		{
			SIZE_T FreeSize = CurrentFreeNode->FreeSize;
			fprintf(stderr, "  FreeNode[0x%016llx] FreeSize: %lu\n", reinterpret_cast<uint64>(CurrentFreeNode), FreeSize);

			CurrentFreeNode = CurrentFreeNode->Next;
		}

		CurrentBlock = CurrentBlock->Next;
	}
	fprintf(stderr, "\n");
}

bool FMallocLinked::IsInternallyThreadSafe() const
{
	return true;
}

const TCHAR* FMallocLinked::GetDescriptiveName()
{
	return TEXT("FMallocLinked");
}

bool FMallocLinked::OwnsPointer(void* Ptr) const
{
	FBlock* CurrentBlock = Blocks;
	while (CurrentBlock != nullptr)
	{
		if (Ptr >= CurrentBlock && Ptr <= AddByteOffsetToPointer(CurrentBlock, CurrentBlock->BlockSize))
		{
			return true;
		}

		CurrentBlock = CurrentBlock->Next;
	}

	return false;
}

FForkPageProtector& FForkPageProtector::Get()
{
	static FForkPageProtector PageProtector;
	return PageProtector;
}

FForkPageProtector::~FForkPageProtector()
{
	close(ProtectedPagesFileFD);
}

void FForkPageProtector::AddMemoryRegion(void* Address, uint64 Size)
{
	if (LIKELY(!FPlatformMemory::HasForkPageProtectorEnabled()) || FForkProcessHelper::IsForkedChildProcess())
	{
		return;
	}

	FScopeLock Lock(&ProtectedRangesSection);
	ProtectedAddresses.Emplace({reinterpret_cast<uint64>(Address), Size});
}

void FForkPageProtector::FreeMemoryRegion(void* Address)
{
	if (LIKELY(!FPlatformMemory::HasForkPageProtectorEnabled()) || FForkProcessHelper::IsForkedChildProcess())
	{
		return;
	}

	FScopeLock Lock(&ProtectedRangesSection);
	for (ProtectedMemoryRange& MemoryRegion : ProtectedAddresses)
	{
		if (MemoryRegion.Address == reinterpret_cast<uint64>(Address))
		{
			// just set the low bit to 1, to mark it as free'd
			MemoryRegion.Address = (MemoryRegion.Address & ~0x1) | 0x1;
		}
	}
}

void ProtectedPagesCrashHandler(int32 Signal, siginfo_t* Info, void* Context)
{
	if (FForkPageProtector::Get().HandleNewCrashAddress(Info->si_addr))
	{
		// if we were able to handle the new crash address lets continue
		// if fail we will treat it like a normal crash
		return;
	}

	FForkPageProtector::Get().UnProtectMemoryRegions();

	PlatformCrashHandler(Signal, Info, Context);
}

void FForkPageProtector::SetupSignalHandler()
{
	struct sigaction Action;
	FMemory::Memzero(Action);
	sigfillset(&Action.sa_mask);
	Action.sa_flags = SA_SIGINFO | SA_RESTART | SA_ONSTACK;
	Action.sa_sigaction = ProtectedPagesCrashHandler;

	// We only need to override the SIGSEGV signal handler as protected pages from mprotect will only generate SIGSEGV signals
	sigaction(SIGSEGV, &Action, nullptr);
}

void FForkPageProtector::ProtectMemoryRegions()
{
	if (!FPlatformMemory::HasForkPageProtectorEnabled() || !FForkProcessHelper::IsForkedChildProcess())
	{
		return;
	}

	UE_LOGF(LogHAL, Warning, "Protect Page Finder has been enabled and is about to protect pages. Output location:");
	UE_LOGF(LogHAL, Warning, "'%ls'", *FPaths::ConvertRelativePathToFull(GetOutputFileLocation()));

	// Setup our new signal handler before we protect *any* pages
	if (!bSetupSignalHandler)
	{
		bSetupSignalHandler = true;

		SetupSignalHandler();
	}

	FScopeLock Lock(&ProtectedRangesSection);
	for (ProtectedMemoryRange& MemoryRange : ProtectedAddresses)
	{
		if (((MemoryRange.Address & 0x1) != 0x1) && (MemoryRange.Address % FPlatformMemory::GetConstants().PageSize == 0))
		{
			// Just in case cover a memory region that is also marked EXEC just throw it on for all
			if (mprotect(reinterpret_cast<void*>(MemoryRange.Address), MemoryRange.Size, PROT_READ | PROT_EXEC) < 0)
			{
				fprintf(stderr, "Failed to mprotect region: %p %llu (%i %s)\n", reinterpret_cast<void*>(MemoryRange.Address), MemoryRange.Size, errno, strerror(errno));
			}
		}
	}

	SetupOutputFile();
}

void FForkPageProtector::UnProtectMemoryRegions()
{
	if (!FPlatformMemory::HasForkPageProtectorEnabled())
	{
		return;
	}

	FScopeLock Lock(&ProtectedRangesSection);
	for (ProtectedMemoryRange& MemoryRange : ProtectedAddresses)
	{
		if (((MemoryRange.Address & 0x1) != 0x1) && (MemoryRange.Address % FPlatformMemory::GetConstants().PageSize == 0))
		{
			// Just in case cover a memory region that is also marked EXEC just throw it on for all
			if (mprotect(reinterpret_cast<void*>(MemoryRange.Address), MemoryRange.Size, PROT_READ | PROT_WRITE | PROT_EXEC) < 0)
			{
				fprintf(stderr, "Failed to mprotect region: %p %llu (%i %s)\n", reinterpret_cast<void*>(MemoryRange.Address), MemoryRange.Size, errno, strerror(errno));
			}
		}
	}
}

bool FForkPageProtector::HandleNewCrashAddress(void* CrashAddress)
{
	if (!FPlatformMemory::HasForkPageProtectorEnabled())
	{
		return false;
	}

	// TODO deal with threading issue
	if (LastCrashAddress == CrashAddress)
	{
		return false;
	}

	LastCrashAddress = CrashAddress;

	// Align the crash addess to the nearest left most page boundary
	uint64 PageAlignedAddress = reinterpret_cast<uint64>(CrashAddress) & ~(FPlatformMemory::GetConstants().PageSize - 1);
	//TODO: check return code here, it can fail to change protection
	mprotect(reinterpret_cast<void*>(PageAlignedAddress), FPlatformMemory::GetConstants().PageSize, PROT_READ | PROT_WRITE);

	return DumpCallstackInfoToFile();
}

const FString& FForkPageProtector::GetOutputFileLocation()
{
	// TODO maybe should try to keep at least N preserved before TRUNC'ing them
	static const FString OutputFullPath = FPaths::ProfilingDir() / TEXT("ProtectedPageHits.propg");

	return OutputFullPath;
}

void FForkPageProtector::SetupOutputFile()
{
	const FString& DefaultOutputFullPath = GetOutputFileLocation();

	// Need to manually setup both the Saved and Profiling dir if they dont exists
	// Need to avoid platform abstraction as checking if Dir Exists seems to hit protected memory
	if (mkdir(TCHAR_TO_ANSI(*FPaths::ProjectSavedDir()), 0775) == -1)
	{
		// something else happened, failure
		if (errno != EEXIST)
		{
			return;
		}
	}

	if (mkdir(TCHAR_TO_ANSI(*FPaths::ProfilingDir()), 0775) == -1)
	{
		// something else happened, failure
		if (errno != EEXIST)
		{
			return;
		}
	}

	ProtectedPagesFileFD = open(TCHAR_TO_ANSI(*DefaultOutputFullPath), O_CREAT | O_RDWR | O_TRUNC, S_IRUSR | S_IWUSR);
}

bool FForkPageProtector::DumpCallstackInfoToFile()
{
	if (ProtectedPagesFileFD == -1)
	{
		fprintf(stderr, "Failed to open ProtectedPageHits.progpg, likely to cause issues\n");
		UnProtectMemoryRegions();

		return false;
	}

	const SIZE_T StackTraceSize = 65535;
	ANSICHAR StackTrace[StackTraceSize] = {};

	int32 IgnoreCount = 2;
	FPlatformStackWalk::StackWalkAndDump(StackTrace, StackTraceSize, IgnoreCount);
	uint64 Hash = CityHash64(StackTrace, FCStringAnsi::Strlen(StackTrace));

	// we can have multiple threads coming into this signal handler, need to make sure
	// that only 1 thread touches CallstackHashCount at a time.
	static volatile sig_atomic_t SignalBeingHandled = 0;

	while (!__sync_bool_compare_and_swap(&SignalBeingHandled, 0, 1)) { sched_yield(); }

	ON_SCOPE_EXIT { FPlatformAtomics::InterlockedDecrement(&SignalBeingHandled); };

	CallstackHashData* Data = CallstackHashCount.Find(Hash);

	if (Data == nullptr)
	{
		SSIZE_T BytesWritten = 0U;
		SIZE_T ActualStackTraceSize = FCStringAnsi::Strlen(StackTrace);
		uint64 ExpectedBytesWritten = sizeof(CallstackHashData::Count) + sizeof(uint64) + ActualStackTraceSize + 1;

		CallstackHashData NewData;
		NewData.Count = 1;
		NewData.FileBytesOffset = CurrentFileOffsetBytes;

		BytesWritten += write(ProtectedPagesFileFD, &NewData.Count, sizeof(CallstackHashData::Count));

		BytesWritten += write(ProtectedPagesFileFD, &Hash, sizeof(uint64));
		BytesWritten += write(ProtectedPagesFileFD, StackTrace, ActualStackTraceSize);
		BytesWritten += write(ProtectedPagesFileFD, "\0", 1);

		if (BytesWritten != ExpectedBytesWritten)
		{
			fprintf(stderr, "Failed to write expected number of bytes: %llu only wrote: %lld\n", ExpectedBytesWritten, BytesWritten);
			UnProtectMemoryRegions();

			return false;
		}

		CurrentFileOffsetBytes += BytesWritten;

		CallstackHashCount.Add(Hash, NewData);
	}
	else
	{
		Data->Count++;

		lseek(ProtectedPagesFileFD, Data->FileBytesOffset, SEEK_SET);

		// Using the stored offset into the ProtectedPagesFile we can update the count directly
		SSIZE_T BytesWritten = write(ProtectedPagesFileFD, &(Data->Count), sizeof(CallstackHashData::Count));

		if (BytesWritten < sizeof(CallstackHashData::Count))
		{
			fprintf(stderr, "Failed to write expected number of bytes: %lu only wrote: %lld\n", sizeof(CallstackHashData::Count), BytesWritten);
			UnProtectMemoryRegions();

			return false;
		}

		lseek(ProtectedPagesFileFD, 0, SEEK_END);
	}

	return true;
}

void FForkPageProtector::OverrideGMalloc()
{
	UE::Private::GMalloc = new FMallocLinked(UE::Private::GMalloc);
}

int (*RealPThreadCreate)(pthread_t*, const pthread_attr_t*, void*(*start) (void*), void*);

/*
 * Overall this will only capture statically compiled code
 * if another DSO is loaded *and* that DSO calls pthread_create we will into issues
 */
int pthread_create(pthread_t* thread, const pthread_attr_t* attr, void*(*start_routine) (void*), void* arg)
{
	if (!RealPThreadCreate)
	{
		RealPThreadCreate = (int(*)(pthread_t*, const pthread_attr_t*, void*(*start_routine) (void*), void*))dlsym(RTLD_DEFAULT, "pthread_create");
	}

	// Only do this if we are page protecting and we are the forked child
	if (UNLIKELY(FPlatformMemory::HasForkPageProtectorEnabled()) && FForkProcessHelper::IsForkedChildProcess())
	{
		if (attr == nullptr)
		{
			pthread_attr_t ThreadAttr;

			// pthreads normal default thread size is 8MB, lets stay default there but use our own stackbase
			// this memory is cached under the hood in nptl pthreads so we no longer will own this memory, and is undefined to free once passed through
			static int DefaultThreadStackSize = 1024 * 1024 * 8;
			void* Stackbase = malloc(DefaultThreadStackSize);

			pthread_attr_init(&ThreadAttr);
			pthread_attr_setstack(&ThreadAttr, Stackbase, DefaultThreadStackSize);

			int PThreadRet = RealPThreadCreate(thread, &ThreadAttr, start_routine, arg);

			pthread_attr_destroy(&ThreadAttr);

			return PThreadRet;
		}
	}

	return RealPThreadCreate(thread, attr, start_routine, arg);
}
} // namespace UE
#endif // COMPILE_FORK_PAGE_PROTECTOR
