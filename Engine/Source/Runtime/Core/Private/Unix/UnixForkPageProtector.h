// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "HAL/CriticalSection.h"
#include "HAL/PlatformMemory.h"

#if COMPILE_FORK_PAGE_PROTECTOR
#include "Containers/Map.h"

#include <sys/mman.h>

namespace UE
{
/*
 * Simple dynamic array that can only grow, avoids allocation code
 * directly calls mmap this is to avoid recursive calls when collecting memory address
 */
template <typename T>
class UnixLowLevelDynamicArray
{
public:
	UnixLowLevelDynamicArray()
	{
		Elements = static_cast<T*>(mmap(nullptr, Capacity * sizeof(T), PROT_READ | PROT_WRITE, MAP_PRIVATE | MAP_ANON, -1, 0));
	}

	~UnixLowLevelDynamicArray()
	{
		munmap(Elements, Capacity);
	}

	void Emplace(T&& Elem)
	{
		// MMap most likely failed in the ctor, cant do much with out memory
		if (Elements == nullptr)
		{
			return;
		}

		if (Size + 1 > Capacity)
		{
			SIZE_T OldCapacity = Capacity;
			Capacity *= 2;

			T* NewElements = static_cast<T*>(mmap(nullptr, Capacity * sizeof(T), PROT_READ | PROT_WRITE, MAP_PRIVATE | MAP_ANON, -1, 0));
			T* OldElements = Elements;

			// MMap has failed
			if (NewElements == nullptr)
			{
				return;
			}

			FMemory::Memcpy(NewElements, Elements, Size * sizeof(T));
			Elements = MoveTemp(NewElements);

			munmap(OldElements, OldCapacity);
		}

		Elements[Size] = MoveTemp(Elem);
		Size++;
	}

	T* begin() const
	{
		return Elements;
	}

	T* end() const
	{
		return Elements + Size;
	}

	T& Get(SIZE_T Index)
	{
		return *Elements[Index];
	}

	SIZE_T GetSize()
	{
		return Size;
	}

private:
	SIZE_T Capacity = 1000U;
	SIZE_T Size     = 0U;
	T* Elements     = nullptr;
};

namespace
{
	struct FBlock;
}

/*
 * Linked list allocator that knows if a given pointer was created by this allocator else a previous allocator
 *
 * Maintains a linked list of Blocks and each Block has linked list of FreeNodes
 *
 */
class FMallocLinked : public FMalloc
{
public:
	explicit FMallocLinked(FMalloc* InPreviousMalloc);

	virtual ~FMallocLinked();

    virtual void* Malloc(SIZE_T Size, uint32 Alignment) override;
    virtual void* Realloc(void* Ptr, SIZE_T NewSize, uint32 Alignment) override;
    virtual void Free(void* Ptr) override;
    virtual const TCHAR * GetDescriptiveName() override;
	virtual bool IsInternallyThreadSafe() const override;

	void DebugVisualize();

private:
	bool OwnsPointer(void* Ptr) const;

	FMalloc* PreviousMalloc = nullptr;
	FBlock* Blocks = nullptr;

	FCriticalSection AllocatorMutex;
};

class FForkPageProtector
{
public:
	static FForkPageProtector& Get();

	/* 
	 * These functions are used on the parent to collect memory regions, as well as keep track of ones that were freed
	 */
	void AddMemoryRegion(void* Address, uint64 Size);
	void FreeMemoryRegion(void* Address);

	/*
	 * This should be used right after fork protect all the memory regions collected above
	 * some may be missed, further consideration may be needed here
	 */
	void ProtectMemoryRegions();

	/*
	 * This should be used if all pages need to be reset to being unprotected
	 */
	void UnProtectMemoryRegions();

	/* 
	 * This function should only be called in the signal handler *after* the ProtectMemoryRegions function has been called
	 *
	 *  Given a CrashAddress we will move that addresses page back to READ/WRITE
	 *  As well as store information about the current callstack and other data
	 *
	 * Returns ture if the CrashAddress was handle
	 *   if false this means some internal error or the same CrashAddress was sent twice in a row
	*/
	bool HandleNewCrashAddress(void* CrashAddress);

	/* 
	 * Overrides GMalloc with our own allocator so we don't stomp on possible protected pages when allocating
	 */
	static void OverrideGMalloc();

private:
	FForkPageProtector() = default;
	~FForkPageProtector();

	void SetupOutputFile();

	bool DumpCallstackInfoToFile();

	void SetupSignalHandler();

	const FString& GetOutputFileLocation();

	// The lower bit of the address will be set to 1 if free'ed
	struct ProtectedMemoryRange
	{
		uint64 Address = 0U;
		uint64 Size    = 0U;
	};

	FCriticalSection ProtectedRangesSection;
	UnixLowLevelDynamicArray<ProtectedMemoryRange> ProtectedAddresses;

	struct CallstackHashData
	{
		uint32 Count           = 0U;
		uint64 FileBytesOffset = 0U;
	};
	
	TMap<uint64, CallstackHashData> CallstackHashCount;

	int ProtectedPagesFileFD      = -1;
	uint64 CurrentFileOffsetBytes = 0U;
	void* LastCrashAddress        = nullptr;
	bool bSetupSignalHandler      = false;
};
#else
namespace UE
{
class FForkPageProtector
{
public:
	static FForkPageProtector& Get()
	{
		static FForkPageProtector PageProtector;
		return PageProtector;
	}

	void AddMemoryRegion(void* Address, uint64 Size) {}
	void FreeMemoryRegion(void* Address) {}
	void ProtectMemoryRegions() {}
	void UnProtectMemoryRegions() {}
	bool HandleNewCrashAddress(void* CrashAddress) { return false; }

	static void OverrideGMalloc() {}
};
#endif // COMPILE_FORK_PAGE_PROTECTOR
} // namespace UE
