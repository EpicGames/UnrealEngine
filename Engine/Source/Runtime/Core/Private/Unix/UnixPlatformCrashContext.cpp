// Copyright Epic Games, Inc. All Rights Reserved.

#include "Unix/UnixPlatformCrashContext.h"
#include "Containers/StringConv.h"
#include "HAL/PlatformStackWalk.h"
#include "HAL/PlatformTime.h"
#include "HAL/PlatformProcess.h"
#include "HAL/PlatformOutputDevices.h"
#include "Logging/LogMacros.h"
#include "CoreGlobals.h"
#include "HAL/FileManager.h"
#include "Misc/Parse.h"
#include "Misc/CommandLine.h"
#include "Misc/Paths.h"
#include "Misc/FileHelper.h"
#include "Delegates/IDelegateInstance.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/Guid.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Misc/OutputDeviceError.h"
#include "Misc/OutputDeviceArchiveWrapper.h"
#include "Containers/Ticker.h"
#include "Misc/FeedbackContext.h"
#include "Misc/App.h"
#include "Misc/EngineVersion.h"
#include "Misc/StringBuilder.h"
#include "HAL/PlatformMallocCrash.h"
#include "Unix/UnixPlatformRealTimeSignals.h"
#include "Unix/UnixPlatformRunnableThread.h"
#include "HAL/ExceptionHandling.h"
#include "HAL/ThreadManager.h"
#include "Stats/Stats.h"
#include "HAL/ThreadHeartBeat.h"
#include "BuildSettings.h"

#include <sys/mman.h>

#include <atomic>

extern CORE_API bool GIsGPUCrashed;

static TAutoConsoleVariable<bool> CVarUnixCaptureAllThreadStacks(
	TEXT("unix.CaptureAllThreadStacksOnCrash"),
	true,
	TEXT("When true, capture callstacks from all threads during crash reporting.\n")
	TEXT("Set to false to disable if this causes issues such as hangs or recursive crashes."));

FString DescribeSignal(int32 Signal, siginfo_t* Info, ucontext_t *Context)
{
	FString ErrorString;

#define HANDLE_CASE(a,b) case a: ErrorString += TEXT(#a ": " b); break;

	switch (Signal)
	{
	case 0:
		// No signal - used for initialization stacktrace on non-fatal errors (ex: ensure)
		break;
	case SIGSEGV:
#ifdef __x86_64__	// architecture-specific
		if (Context && (Context->uc_mcontext.gregs[REG_TRAPNO] == 13))
		{
			ErrorString += FString::Printf(TEXT("SIGSEGV: unaligned memory access at address 0x%016llx (SIMD vectors?)"), (uint64)Info->si_addr);
		}
		else
		{
			ErrorString += FString::Printf(TEXT("SIGSEGV: invalid attempt to %s memory at address 0x%016llx"),
				(Context != nullptr) ? ((Context->uc_mcontext.gregs[REG_ERR] & 0x2) ? TEXT("write") : TEXT("read")) : TEXT("access"), (uint64)Info->si_addr);
		}
#else
		ErrorString += FString::Printf(TEXT("SIGSEGV: invalid attempt to access memory at address 0x%016llx"), (uint64)Info->si_addr);
#endif // __x86_64__
		break;
	case SIGBUS:
		ErrorString += FString::Printf(TEXT("SIGBUS: invalid attempt to access memory at address 0x%016llx"), (uint64)Info->si_addr);
		break;
	case SIGSYS:
		ErrorString += FString::Printf(TEXT("SIGSYS: non-existent or invalid system call invoked %i"), Info->si_syscall);
		break;

		HANDLE_CASE(SIGINT, "program interrupted")
		HANDLE_CASE(SIGQUIT, "user-requested crash")
		HANDLE_CASE(SIGILL, "illegal instruction")
		HANDLE_CASE(SIGTRAP, "trace trap")
		HANDLE_CASE(SIGABRT, "abort() called")
		HANDLE_CASE(SIGFPE, "floating-point exception")
		HANDLE_CASE(SIGKILL, "program killed")
		HANDLE_CASE(SIGPIPE, "write on a pipe with no reader")
		HANDLE_CASE(SIGTERM, "software termination signal")
		HANDLE_CASE(SIGSTOP, "stop")

	default:
		ErrorString += FString::Printf(TEXT("Signal %d (unknown)"), Signal);
	}

	return ErrorString;
#undef HANDLE_CASE
}

/** Implement platform specific static cleanup function */
void FGenericCrashContext::CleanupPlatformSpecificFiles()
{
}

__thread siginfo_t FUnixCrashContext::FakeSiginfoForDiagnostics;

FUnixCrashContext::~FUnixCrashContext()
{
	if (BacktraceSymbols)
	{
		// glibc uses malloc() to allocate this, and we only need to free one pointer, see http://www.gnu.org/software/libc/manual/html_node/Backtraces.html
		free(BacktraceSymbols);
		BacktraceSymbols = NULL;
	}
}

void FUnixCrashContext::InitFromSignal(int32 InSignal, siginfo_t* InInfo, void* InContext)
{
	Signal = InSignal;
	Info = InInfo;
	Context = reinterpret_cast< ucontext_t* >( InContext );

	FCString::StrncatTruncateDest(SignalDescription, UE_ARRAY_COUNT( SignalDescription ), *DescribeSignal(Signal, Info, Context));
}

void FUnixCrashContext::InitFromDiagnostics(const void* InAddress)
{
	Signal = SIGTRAP;

	FakeSiginfoForDiagnostics.si_signo = SIGTRAP;
	FakeSiginfoForDiagnostics.si_code = TRAP_TRACE;
	FakeSiginfoForDiagnostics.si_addr = const_cast<void *>(InAddress);
	Info = &FakeSiginfoForDiagnostics;

	Context = nullptr;

	// set signal description to a more human-readable one for ensures
	FCString::Strncpy(SignalDescription, ErrorMessage, UE_ARRAY_COUNT(SignalDescription));

	// only need the first string
	for (int Idx = 0; Idx < UE_ARRAY_COUNT(SignalDescription); ++Idx)
	{
		if (SignalDescription[Idx] == TEXT('\n'))
		{
			SignalDescription[Idx] = 0;
			break;
		}
	}
}

volatile sig_atomic_t GEnteredSignalHandler = 0;

/**
 * Handles graceful termination. Gives time to exit gracefully, but second signal will quit immediately.
 */
void GracefulTerminationHandler(int32 Signal, siginfo_t* Info, void* Context)
{
	GEnteredSignalHandler = 1;

	// Possibly better to add a 2nd function if this is more required. Since we are in the Core module lets see if this is only needed here
	extern bool GShouldRequestExit;

	// do not flush logs at this point; this can result in a deadlock if the signal was received while we were holding lock in the malloc (flushing allocates memory)
	if( !IsEngineExitRequested() && !GShouldRequestExit )
	{
		FPlatformMisc::RequestExitWithStatus(false, static_cast<uint8>(128 + Signal));	// Keeping the established shell practice of returning 128 + signal for terminations by signal. Allows to distinguish SIGINT/SIGTERM/SIGHUP.
	}
	else
	{
		FPlatformMisc::RequestExit(true, TEXT("UnixPlatformCrashContext.GracefulTerminationHandler"));
	}

	GEnteredSignalHandler = 0;
}

void CreateExceptionInfoString(int32 Signal, siginfo_t* Info, ucontext_t *Context)
{
	FString ErrorString = "Unhandled Exception: ";
	ErrorString += DescribeSignal(Signal, Info, Context);
	FCString::Strncpy(GErrorExceptionDescription, *ErrorString, FMath::Min(ErrorString.Len() + 1, (int32)UE_ARRAY_COUNT(GErrorExceptionDescription)));
}

namespace
{
	/** 
	 * Write a line of UTF-8 to a file
	 */
	void WriteLine(FArchive* ReportFile, const ANSICHAR* Line = NULL)
	{
		if( Line != NULL )
		{
			int64 StringBytes = FCStringAnsi::Strlen(Line);
			ReportFile->Serialize(( void* )Line, StringBytes);
		}

		// use Windows line terminator
		static ANSICHAR WindowsTerminator[] = "\r\n";
		ReportFile->Serialize(WindowsTerminator, 2);
	}

	/**
	 * Serializes UTF string to UTF-16
	 */
	void WriteUTF16String(FArchive* ReportFile, const TCHAR * UTFString4BytesChar, uint32 NumChars)
	{
		check(UTFString4BytesChar != NULL || NumChars == 0);

		for (uint32 Idx = 0; Idx < NumChars; ++Idx)
		{
			ReportFile->Serialize(const_cast< TCHAR* >( &UTFString4BytesChar[Idx] ), 2);
		}
	}

	/** 
	 * Writes UTF-16 line to a file
	 */
	void WriteLine(FArchive* ReportFile, const TCHAR* Line)
	{
		if( Line != NULL )
		{
			int64 NumChars = FCString::Strlen(Line);
			WriteUTF16String(ReportFile, Line, NumChars);
		}

		// use Windows line terminator
		static TCHAR WindowsTerminator[] = TEXT("\r\n");
		WriteUTF16String(ReportFile, WindowsTerminator, 2);
	}
}

/** 
 * Write all the data mined from the minidump to a text file
 */
void FUnixCrashContext::GenerateReport(const FString & DiagnosticsPath) const
{
	FArchive* ReportFile = IFileManager::Get().CreateFileWriter(*DiagnosticsPath);
	if (ReportFile != NULL)
	{
		FString Line;

		WriteLine(ReportFile, "Generating report for minidump");
		WriteLine(ReportFile);

		Line = FString::Printf(TEXT("Application version %d.%d.%d.0" ), FEngineVersion::Current().GetMajor(), FEngineVersion::Current().GetMinor(), FEngineVersion::Current().GetPatch());
		WriteLine(ReportFile, TCHAR_TO_UTF8(*Line));

		Line = FString::Printf(TEXT(" ... built from changelist %d"), FEngineVersion::Current().GetChangelist());
		WriteLine(ReportFile, TCHAR_TO_UTF8(*Line));
		WriteLine(ReportFile);

		utsname UnixName;
		if (uname(&UnixName) == 0)
		{
			Line = FString::Printf(TEXT( "OS version %s %s (network name: %s)" ), UTF8_TO_TCHAR(UnixName.sysname), UTF8_TO_TCHAR(UnixName.release), UTF8_TO_TCHAR(UnixName.nodename));
			WriteLine(ReportFile, TCHAR_TO_UTF8(*Line));	
			Line = FString::Printf( TEXT( "Running %d %s processors (%d logical cores)" ), FPlatformMisc::NumberOfCores(), UTF8_TO_TCHAR(UnixName.machine), FPlatformMisc::NumberOfCoresIncludingHyperthreads());
			WriteLine(ReportFile, TCHAR_TO_UTF8(*Line));
		}
		else
		{
			Line = FString::Printf(TEXT("OS version could not be determined (%d, %s)"), errno, UTF8_TO_TCHAR(strerror(errno)));
			WriteLine(ReportFile, TCHAR_TO_UTF8(*Line));	
			Line = FString::Printf( TEXT( "Running %d unknown processors" ), FPlatformMisc::NumberOfCores());
			WriteLine(ReportFile, TCHAR_TO_UTF8(*Line));
		}
		Line = FString::Printf(TEXT("Exception was \"%s\""), SignalDescription);
		WriteLine(ReportFile, TCHAR_TO_UTF8(*Line));
		WriteLine(ReportFile);

		WriteLine(ReportFile, "<SOURCE START>");
		WriteLine(ReportFile, "<SOURCE END>");
		WriteLine(ReportFile);

		WriteLine(ReportFile, "<CALLSTACK START>");
		WriteLine(ReportFile, MinidumpCallstackInfo);
		WriteLine(ReportFile, "<CALLSTACK END>");
		WriteLine(ReportFile);

		WriteLine(ReportFile, "0 loaded modules");

		WriteLine(ReportFile);

		Line = FString::Printf(TEXT("Report end!"));
		WriteLine(ReportFile, TCHAR_TO_UTF8(*Line));

		ReportFile->Close();
		delete ReportFile;
	}
}

void FUnixCrashContext::CaptureStackTrace(void* ErrorProgramCounter)
{
	// Only do work the first time this function is called - this is mainly a carry over from Windows where it can be called multiple times, left intact for extra safety.
	if (!bCapturedBacktrace)
	{
		bCapturedBacktrace = true;

		static const SIZE_T StackTraceSize = 65535;
		static ANSICHAR StackTrace[StackTraceSize];
		StackTrace[0] = 0;

		CapturePortableCallStack(ErrorProgramCounter, this);

		// Walk the stack and dump it to the allocated memory (do not ignore any stack frames to be consistent with check()/ensure() handling)
		FPlatformStackWalk::StackWalkAndDump( StackTrace, StackTraceSize, ErrorProgramCounter, this);

#if !PLATFORM_LINUX
		printf("StackTrace:\n%s\n", StackTrace);
#endif

		FCString::StrncatTruncateDest( GErrorHist, UE_ARRAY_COUNT(GErrorHist), UTF8_TO_TCHAR(StackTrace) );
		CreateExceptionInfoString(Signal, Info, Context);
	}
}

void FUnixCrashContext::CaptureThreadStackTrace(uint32_t ThreadId)
{
	// Only do work the first time this function is called - this is mainly a carry over from Windows where it can be called multiple times, left intact for extra safety.
	if (!bCapturedBacktrace)
	{
		bCapturedBacktrace = true;

		CaptureThreadPortableCallStack(ThreadId, this);

		// The crash report XML has an element <CallStack>. However, CrashReportClient will run after the UE process generates the report. It will overwrite this
		//  part of the report with whatever is in the CALLSTACK section of Diagnostics.txt. Because of this we have to call APIs here to ensure that MiniDumpCallstackInfo
		//  gets filled out. So, we iterate through the portable callstack for the thread and call ProgramCounterToHumanReadableString passing the CrashContext (this).
		static const SIZE_T StackTraceSize = 65535;
		static ANSICHAR StackTrace[StackTraceSize];
		StackTrace[0] = 0;
		for ( int i=0; i<CallStack.Num(); i++ )
		{
			FPlatformStackWalk::ProgramCounterToHumanReadableString( i, CallStack[i].BaseAddress + CallStack[i].Offset, StackTrace, StackTraceSize, this );
			FCStringAnsi::StrncatTruncateDest(StackTrace, (int32)StackTraceSize, LINE_TERMINATOR_ANSI);
		}

		// We do not set the ExceptionInfo string here as it just gets details for the exact callsite,
		//  and this function by definition is just capturing the state of some other specific thread
		//  (and not some instrumentation point or instruction pointer that generated a signal)
	}
}

// Defined in UnixPlatformStackWalk.cpp - prevents deliberate crash on timeout during crash handling
extern bool GUnixStackWalkNoCrashOnTimeout;

void FUnixCrashContext::CaptureAllThreadContexts()
{
#if PLATFORM_SUPPORTS_ALL_THREAD_BACKTRACES
	if (!CVarUnixCaptureAllThreadStacks.AsVariable()->GetBool())
	{
		return;
	}

	const uint32 CrashThreadId = FPlatformTLS::GetCurrentThreadId();

	// Reserve upfront to minimize allocations during crash handling.
	// ~1KB per thread XML entry, 128 threads max = ~128KB.
	AllThreadContextsString.Reserve(128 * 1024);

	// Prevent recursive crash if a thread is unresponsive during stack capture
	GUnixStackWalkNoCrashOnTimeout = true;

	FThreadManager::Get().ForEachThreadStackBackTrace(
		[this, CrashThreadId](uint32 ThreadId, const TCHAR* ThreadName, const TConstArrayView<uint64>& StackTrace)
		{
			TArray<FCrashStackFrame> PortableStack;
			if (ThreadId == CrashThreadId && CallStack.Num() > 0)
			{
				// For the crashing thread, use the already-captured CallStack which has
				// better accuracy from the signal context than a re-capture would
				PortableStack = CallStack;
			}
			else
			{
				GetPortableCallStack(StackTrace.GetData(), StackTrace.Num(), PortableStack);
			}

			// Skip threads where we failed to capture a callstack (e.g. thread already
			// exited, or the RT signal could not be delivered). This matches Windows
			// behavior where such threads don't appear with empty stacks.
			if (PortableStack.Num() == 0 && ThreadId != CrashThreadId)
			{
				return true;
			}

			// Format as XML and append
			int32 MaxModuleNameLen = 0;
			for (const FCrashStackFrame& StFrame : PortableStack)
			{
				MaxModuleNameLen = FMath::Max(MaxModuleNameLen, StFrame.ModuleName.Len());
			}

			AllThreadContextsString += TEXT("<Thread>");
			{
				AllThreadContextsString += TEXT("<CallStack>");
				FString CallstackStr;
				for (const FCrashStackFrame& StFrame : PortableStack)
				{
					CallstackStr += FString::Printf(TEXT("%-*s 0x%016llx + %-16llx"), MaxModuleNameLen + 1, *StFrame.ModuleName, StFrame.BaseAddress, StFrame.Offset);
					CallstackStr += LINE_TERMINATOR;
				}
				AppendEscapedXMLString(AllThreadContextsString, *CallstackStr);
				AllThreadContextsString += TEXT("</CallStack>");
				AllThreadContextsString += LINE_TERMINATOR;
			}
			AllThreadContextsString += FString::Printf(TEXT("<IsCrashed>%s</IsCrashed>"), ThreadId == CrashThreadId ? TEXT("true") : TEXT("false"));
			AllThreadContextsString += LINE_TERMINATOR;
			AllThreadContextsString += TEXT("<Registers></Registers>");
			AllThreadContextsString += LINE_TERMINATOR;
			AllThreadContextsString += FString::Printf(TEXT("<ThreadID>%d</ThreadID>"), ThreadId);
			AllThreadContextsString += LINE_TERMINATOR;
			AllThreadContextsString += TEXT("<ThreadName>");
			AppendEscapedXMLString(AllThreadContextsString, ThreadName);
			AllThreadContextsString += TEXT("</ThreadName>");
			AllThreadContextsString += LINE_TERMINATOR;
			AllThreadContextsString += TEXT("</Thread>");
			AllThreadContextsString += LINE_TERMINATOR;

			return true;
		});

	GUnixStackWalkNoCrashOnTimeout = false;
#endif
}

bool FUnixCrashContext::GetPlatformAllThreadContextsString(FString& OutStr) const
{
	OutStr = AllThreadContextsString;
	AllThreadContextsString.Empty();
	return !OutStr.IsEmpty();
}

void FUnixCrashContext::GetPortableCallStack(const uint64* StackFrames, int32 NumStackFrames, TArray<FCrashStackFrame>& OutCallStack) const
{
	// Update the callstack with offsets from each module
	OutCallStack.Reset(NumStackFrames);
	for (int32 Idx = 0; Idx < NumStackFrames; Idx++)
	{
		const uint64 StackFrame = StackFrames[Idx];

		// Try to find the module containing this stack frame
		const FStackWalkModuleInfo* FoundModule = nullptr;

		Dl_info DylibInfo;
		int32 Result = dladdr((const void*)StackFrame, &DylibInfo);
		if (Result != 0)
		{
			ANSICHAR* DylibPath = (ANSICHAR*)DylibInfo.dli_fname;
			ANSICHAR* DylibName = FCStringAnsi::Strrchr(DylibPath, '/');
			if (DylibName)
			{
				DylibName += 1;
			}
			else
			{
				DylibName = DylibPath;
			}
			OutCallStack.Add(FCrashStackFrame(FPaths::GetBaseFilename(DylibName), (uint64)DylibInfo.dli_fbase, StackFrame - (uint64)DylibInfo.dli_fbase));
		}
		else
		{
			OutCallStack.Add(FCrashStackFrame(TEXT("Unknown"), 0, StackFrame));
		}
	}
}

#ifndef SERVER_MAX_CONCURRENT_REPORTS
	#define SERVER_MAX_CONCURRENT_REPORTS 1
#endif

namespace UnixCrashReporterTracker
{
	enum class SlotStatus : uint32
	{
		Available,
		Spawning,
		Uploading,
		Closing,
		Killing
	};

	struct CrashReporterProcess
	{
		FProcHandle Process;
		std::atomic<UnixCrashReporterTracker::SlotStatus> Status;
	};

	// Matching MaxPreviousErrorsToTrack = 4 in AssertionMacros.cpp
	CrashReporterProcess Processes[4];

	/** Maximum index in the process slot array */
	uint32 MaxProcessSlots;

	/** Number of active processes uploading their data at the moment */
	std::atomic<uint32> NumUploadingProcesses(0);

	bool Tick(float DeltaTime)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_UnixCrashReporterTracker_Tick);

		uint32 NumActiveProcess = NumUploadingProcesses.load(std::memory_order_relaxed);

		if (NumActiveProcess > 0)
		{
			for (uint32 ProcessNumber = 0; ProcessNumber < MaxProcessSlots; ++ProcessNumber)
			{
				CrashReporterProcess& CurrentSlot = Processes[ProcessNumber];

				SlotStatus IsUploading = SlotStatus::Uploading;
				const SlotStatus IsClosing = SlotStatus::Closing;

				// Test if an uploading process is finished
				if (CurrentSlot.Status.compare_exchange_weak(IsUploading, IsClosing))
				{
					if (!FPlatformProcess::IsProcRunning(CurrentSlot.Process))
					{
						FPlatformProcess::CloseProc(CurrentSlot.Process);
						CurrentSlot.Process = FProcHandle();

						--NumUploadingProcesses;
						CurrentSlot.Status = SlotStatus::Available;
					}
					else
					{
						CurrentSlot.Status = SlotStatus::Uploading;
					}
				}
			}
		}

		// tick again
		return true;
	}

	void PreInit()
	{
		for (CrashReporterProcess& CurrentSlot : Processes)
		{
			CurrentSlot.Status = SlotStatus::Available;
		}

		uint32 ActiveProcessSlots = UE_ARRAY_COUNT(Processes);

		// Lower the amount of concurrent reports on servers to limit the spike in cpu/memory when sending those reports
		if (IsRunningDedicatedServer())
		{
			ActiveProcessSlots = FMath::Min((uint32)SERVER_MAX_CONCURRENT_REPORTS, ActiveProcessSlots);
		}

		// Set the valid max index to iterate over
		UnixCrashReporterTracker::MaxProcessSlots = ActiveProcessSlots;

        // Register our Tick function
		FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateStatic(&UnixCrashReporterTracker::Tick), 1.f);
	}

	/**
	 * Waits for the proc with timeout (busy loop, workaround for platform abstraction layer not exposing this)
	 *
	 * @param Proc proc handle to wait for
	 * @param TimeoutInSec timeout in seconds
	 * @param SleepIntervalInSec sleep interval (the smaller the more CPU we will eat, but the faster we will detect the program exiting)
	 *
	 * @return true if exited cleanly, false if timeout has expired
	 */
	bool WaitForProcWithTimeout(FProcHandle Proc, const double TimeoutInSec, const double SleepIntervalInSec)
	{
		double StartSeconds = FPlatformTime::Seconds();
		for (;;)
		{
			if (!FPlatformProcess::IsProcRunning(Proc))
			{
				break;
			}

			if (FPlatformTime::Seconds() - StartSeconds > TimeoutInSec)
			{
				return false;
			}

			FPlatformProcess::Sleep(static_cast<float>(SleepIntervalInSec));
		};

		return true;
	}

	void RemoveValidCrashReportTickerForChildProcess()
	{
		for (uint32 ProcessNumber = 0; ProcessNumber < UnixCrashReporterTracker::MaxProcessSlots; ++ProcessNumber)
		{
			UnixCrashReporterTracker::Processes[ProcessNumber].Process = FProcHandle();
			UnixCrashReporterTracker::Processes[ProcessNumber].Status = UnixCrashReporterTracker::SlotStatus::Available;
		}

		UnixCrashReporterTracker::NumUploadingProcesses = 0;
	}
}

void FUnixCrashContext::AddPlatformSpecificProperties() const
{
	AddCrashProperty(TEXT("CrashSignal"), Signal);

	ANSICHAR* AnsiSignalName = strsignal(Signal);
	if (AnsiSignalName != nullptr)
	{
		TStringBuilder<32> SignalName;
		SignalName.Append(AnsiSignalName);

		AddCrashProperty(TEXT("CrashSignalName"), *SignalName);
	}
	else
	{
		AddCrashProperty(TEXT("CrashSignalName"), TEXT("Unknown"));
	}
}

void FUnixCrashContext::GenerateCrashInfoAndLaunchReporter() const
{
	const bool bReportingNonCrash = IsTypeContinuable(Type);

	// do not report crashes for tools (particularly for crash reporter itself)
#if !IS_PROGRAM

	// create a crash-specific directory
	FString CrashGuid;
	if (!FParse::Value(FCommandLine::Get(), TEXT("CrashGUID="), CrashGuid) || CrashGuid.Len() <= 0)
	{
		CrashGuid = FGuid::NewGuid().ToString();
	}


	/* Table showing the desired behavior when wanting to start the CRC or not.
	 *  based on an *.ini setting for bSendUnattendedBugReports or bAgreeToCrashUpload and whether or not we are unattended
	 *
	 *  Unattended | AgreeToUpload | SendUnattendedBug || Start CRC
	 *  ------------------------------------------------------------
	 *      1      |       1       |         1         ||     1
	 *      1      |       1       |         0         ||     1
	 *      1      |       0       |         1         ||     1
	 *      1      |       0       |         0         ||     0
	 *      0      |       1       |         1         ||     1
	 *      0      |       1       |         0         ||     1
	 *      0      |       0       |         1         ||     1
	 *      0      |       0       |         0         ||     1
	 *
	 */

	// Suppress the user input dialog if we're running in unattended mode
	bool bUnattended = FApp::IsUnattended() || (!IsInteractiveEnsureMode() && bReportingNonCrash) || IsRunningDedicatedServer();

#if PLATFORM_LINUX
	// On Linux, count not having a X11 display as also running unattended, because CRC will switch to the unattended mode in that case
	if (!bUnattended)
	{
		// see CrashReportClientMainLinux.cpp
		if (getenv("DISPLAY") == nullptr)
		{
			bUnattended = true;
		}
	}
#endif

	bool bImplicitSend = false;
	if (!UE_EDITOR && GConfig && !bReportingNonCrash)
	{
		// Only check if we are in a non-editor build
		GConfig->GetBool(TEXT("CrashReportClient"), TEXT("bImplicitSend"), bImplicitSend, GEngineIni);
	}

	// By default we wont upload unless the *.ini has set this to true
	bool bSendUnattendedBugReports = false;
	if (GConfig)
	{
		GConfig->GetBool(TEXT("/Script/UnrealEd.CrashReportsPrivacySettings"), TEXT("bSendUnattendedBugReports"), bSendUnattendedBugReports, GEditorSettingsIni);
	}

	// Controls if we want analytics in the crash report client
	bool bSendUsageData = true;
	if (GConfig)
	{
		GConfig->GetBool(TEXT("/Script/UnrealEd.AnalyticsPrivacySettings"), TEXT("bSendUsageData"), bSendUsageData, GEditorSettingsIni);
	}

	// If we are not an editor but still want to agree to upload for non-licensee check the settings
	bool bAgreeToCrashUpload = false;
	if (!UE_EDITOR && GConfig)
	{
		GConfig->GetBool(TEXT("CrashReportClient"), TEXT("bAgreeToCrashUpload"), bAgreeToCrashUpload, GEngineIni);
	}

	if (BuildSettings::IsLicenseeVersion() && !UE_EDITOR)
	{
		// do not send unattended reports in licensees' builds except for the editor, where it is governed by the above setting
		bSendUnattendedBugReports = false;
		bAgreeToCrashUpload = false;
		bSendUsageData = false;
	}

	bool bSkipCRC = bUnattended && !bSendUnattendedBugReports && !bAgreeToCrashUpload;

	if (!bSkipCRC)
	{
		const TCHAR* TypeString = TEXT("crash");
		switch(Type)
		{
		case ECrashContextType::Ensure:
			TypeString = TEXT("ensure");
			break;

		case ECrashContextType::Stall:
			TypeString = TEXT("stall");
			break;
		}

		FString CrashInfoFolder = FPaths::Combine(*FPaths::ProjectSavedDir(), TEXT("Crashes"), *FString::Printf(TEXT("%sinfo-%s-pid-%d-%s"), TypeString, FApp::GetProjectName(), getpid(), *CrashGuid));
		FString CrashInfoAbsolute = FPaths::ConvertRelativePathToFull(CrashInfoFolder);
		if (IFileManager::Get().MakeDirectory(*CrashInfoAbsolute, true))
		{
			// generate additional files from providers.
			FAdditionalCrashContextParams Params =
			{
				.CrashedThreadId = GetCrashedThreadId()
			};
			DumpAdditionalContext(*CrashInfoFolder, Params);

			// generate "minidump"
			GenerateReport(FPaths::Combine(*CrashInfoAbsolute, TEXT("Diagnostics.txt")));

			// Introduces a new runtime crash context. Will replace all Windows related crash reporting.
			FString FilePath(CrashInfoFolder);
			FilePath += TEXT("/");
			FilePath += FGenericCrashContext::CrashContextRuntimeXMLNameW;
			SerializeAsXML(*FilePath);

			// copy log
			FString LogSrcAbsolute = FPlatformOutputDevices::GetAbsoluteLogFilename();
			FString LogFolder = FPaths::GetPath(LogSrcAbsolute);
			FString LogFilename = FPaths::GetCleanFilename(LogSrcAbsolute);
			FString LogBaseFilename = FPaths::GetBaseFilename(LogSrcAbsolute);
			FString LogExtension = FPaths::GetExtension(LogSrcAbsolute, true);
			FString LogDstAbsolute = FPaths::Combine(*CrashInfoAbsolute, *LogFilename);
			FPaths::NormalizeDirectoryName(LogDstAbsolute);

			// Flush out the log
			GLog->Flush();

#if !NO_LOGGING
			bool bMemoryOnly = FPlatformOutputDevices::GetLog()->IsMemoryOnly();
			bool bBacklogEnabled = FOutputDeviceRedirector::Get()->HasBacklogForCrash();

			// The minimum free space on drive for saving a crash log
			const uint64 MinDriveSpaceForCrashLog = 250 * 1024 * 1024;
			// Max log file size to copy to report folder as will be filtered before submission to crash reporting backend
			const uint64 MaxFileSizeForCrashLog = 100 * 1024 * 1024;

			// Check whether server has low disk space available
			uint64 TotalDiskSpace = 0;
			uint64 TotalDiskFreeSpace = 0;
			bool bLowDriveSpace = false;
			if (FPlatformMisc::GetDiskTotalAndFreeSpace(*CrashInfoAbsolute, TotalDiskSpace, TotalDiskFreeSpace))
			{
				if (TotalDiskFreeSpace < MinDriveSpaceForCrashLog)
				{
					bLowDriveSpace = true;
				}
			}

			if (!bLowDriveSpace)
			{
				if (bMemoryOnly || bBacklogEnabled)
				{
					FArchive* LogFile = IFileManager::Get().CreateFileWriter(*LogDstAbsolute, FILEWRITE_AllowRead);
					if (LogFile)
					{
						if (bMemoryOnly)
						{
							FPlatformOutputDevices::GetLog()->Dump(*LogFile);
						}
						else
						{
							FOutputDeviceArchiveWrapper Wrapper(LogFile);
							GLog->SerializeBacklogForCrash(&Wrapper);
						}

						LogFile->Flush();
						delete LogFile;
					}
				}
				else
				{
					const bool bReplace = true;
					const bool bEvenIfReadOnly = false;
					const bool bAttributes = false;
					FCopyProgress* const CopyProgress = nullptr;

					// Only copy the log file if it is MaxLogFileSizeForCrashReport or less (note it will be zlib compressed for submission to data router)
					if (IFileManager::Get().FileExists(*LogSrcAbsolute) && IFileManager::Get().FileSize(*LogSrcAbsolute) <= MaxFileSizeForCrashLog)
					{
						static_cast<void>(IFileManager::Get().Copy(*LogDstAbsolute, *LogSrcAbsolute, bReplace, bEvenIfReadOnly, bAttributes, CopyProgress, FILEREAD_AllowWrite, FILEWRITE_AllowRead));	// best effort, so don't care about result: couldn't copy -> tough, no log
					}
					else
					{
						FFileHelper::SaveStringToFile(FString("Log not available, too large for submission to crash reporting backend"), *LogDstAbsolute);
					}
				}
			}
			else if (TotalDiskFreeSpace >= MaxFileSizeForCrashLog)
			{
				FFileHelper::SaveStringToFile(FString("Log not available, server has low available disk space"), *LogDstAbsolute);
			}
#endif // !NO_LOGGING

			// If present, include the crash report config file to pass config values to the CRC
			const TCHAR* CrashConfigFilePath = GetCrashConfigFilePath();
			if (IFileManager::Get().FileExists(CrashConfigFilePath))
			{
				FString CrashConfigFilename = FPaths::GetCleanFilename(CrashConfigFilePath);
				FString CrashConfigDstAbsolute = FPaths::Combine(*CrashInfoAbsolute, *CrashConfigFilename);
				static_cast<void>(IFileManager::Get().Copy(*CrashConfigDstAbsolute, CrashConfigFilePath));	// best effort, so don't care about result
			}
#if WITH_EDITOR
			FString CrashReportClientPath = FPaths::ConvertRelativePathToFull(FPlatformProcess::GenerateApplicationPath(TEXT("CrashReportClientEditor"), EBuildConfiguration::Development));
#else
			FString CrashReportClientPath = FPaths::ConvertRelativePathToFull(FPlatformProcess::GenerateApplicationPath(TEXT("CrashReportClient"), EBuildConfiguration::Development));
#endif
			FString CrashReportLogFilename = LogBaseFilename + TEXT("-CRC") + LogExtension;
			FString CrashReportLogFilepath = FPaths::Combine(*LogFolder, *CrashReportLogFilename);
			FString CrashReportClientArguments = " -Abslog=";
			CrashReportClientArguments += TEXT("\"\"") + CrashReportLogFilepath + TEXT("\"\"");
			CrashReportClientArguments += TEXT(" ");

			// If the editor setting has been disabled to not send analytics extend this to the CRC
			if (!bSendUsageData)
			{
				CrashReportClientArguments += TEXT(" -NoAnalytics ");
			}

			if (bImplicitSend)
			{
				CrashReportClientArguments += TEXT(" -Unattended -ImplicitSend ");
			}
			else if (bUnattended)
			{
				CrashReportClientArguments += TEXT(" -Unattended ");
			}

			// Whether to clean up crash reports after send
			if (IsRunningDedicatedServer() && FParse::Param(FCommandLine::Get(), TEXT("CleanCrashReports")))
			{
				CrashReportClientArguments += TEXT(" -CleanCrashReports ");
			}

			CrashReportClientArguments += TEXT("\"\"") + CrashInfoAbsolute + TEXT("/\"\"");

			// Things can be setup to allow for a global crash handler to capture the core from a crash and allow another process
			// to handle spawning of this process
			bool bStartCRCFromEngineHandler = true;

			if (GConfig)
			{
				GConfig->GetBool(TEXT("CrashReportClient"), TEXT("bStartCRCFromEngineHandler"), bStartCRCFromEngineHandler, GEngineIni);
			}

			if (bReportingNonCrash && bStartCRCFromEngineHandler)
			{
				bool bFoundEmptySlot = false;

				constexpr double kEnsureTimeOut = 45.0;
				constexpr float kEnsureSleepInterval = 0.1f;
				double kTimeOutTimer = 0.0;

				while (!bFoundEmptySlot)
				{
					// Find an empty slot for sending the report
					for( uint32 ProcessIdx=0; ProcessIdx < UnixCrashReporterTracker::MaxProcessSlots; ++ProcessIdx )
					{
						UnixCrashReporterTracker::SlotStatus IsAvailable = UnixCrashReporterTracker::SlotStatus::Available;
						const UnixCrashReporterTracker::SlotStatus IsSpawning = UnixCrashReporterTracker::SlotStatus::Spawning;
						
						// If the slot is available, get exclusive rights by setting it to spawning state
						if (UnixCrashReporterTracker::Processes[ProcessIdx].Status.compare_exchange_weak(IsAvailable, IsSpawning))
						{
							UnixCrashReporterTracker::Processes[ProcessIdx].Process = FPlatformProcess::CreateProc(
								*CrashReportClientPath, *CrashReportClientArguments, true, false, false, NULL, 0, NULL, NULL);
							
							++UnixCrashReporterTracker::NumUploadingProcesses;
							UnixCrashReporterTracker::Processes[ProcessIdx].Status = UnixCrashReporterTracker::SlotStatus::Uploading;

							bFoundEmptySlot = true;
							break;
						}
					}
					
					// All process handles in use: wait for up to 45 seconds on the next available process
					if (!bFoundEmptySlot)
					{
						// If all slots are uploading on a server, skip the report instead of hitching
						if (IsRunningDedicatedServer())
						{
							UE_LOGF(LogCore, Warning, "Too many reports already in progress, skipping upload of this one.");
							bFoundEmptySlot = true;
						}
						else
						{
							FPlatformProcess::Sleep(kEnsureSleepInterval);
							UnixCrashReporterTracker::Tick(0.001f);

							kTimeOutTimer += kEnsureSleepInterval;

							// After waiting 45seconds, kill the process at slot 0 and lose it's information.
							if (kTimeOutTimer >= kEnsureTimeOut)
							{
								UnixCrashReporterTracker::SlotStatus IsUploading = UnixCrashReporterTracker::SlotStatus::Uploading;
								const UnixCrashReporterTracker::SlotStatus IsKilling = UnixCrashReporterTracker::SlotStatus::Killing;

								// Take over this uploading slot and kill it
								if (UnixCrashReporterTracker::Processes[0].Status.compare_exchange_weak(IsUploading, IsKilling))
								{
									UE_LOGF(LogCore, Warning, "Terminated CrashReport process[0]");

									FPlatformProcess::TerminateProc(UnixCrashReporterTracker::Processes[0].Process);
									UnixCrashReporterTracker::Processes[0].Process = FProcHandle();

									--UnixCrashReporterTracker::NumUploadingProcesses;
									UnixCrashReporterTracker::Processes[0].Status = UnixCrashReporterTracker::SlotStatus::Available;
								}
								
							}
						}
					}
				}
			}
			else if (bStartCRCFromEngineHandler)
			{
				// spin here until CrashReporter exits
				FProcHandle RunningProc = FPlatformProcess::CreateProc(*CrashReportClientPath, *CrashReportClientArguments, true, false, false, NULL, 0, NULL, NULL);

				// do not wait indefinitely - can be more generous about the hitch than in ensure() case
				// NOTE: Chris.Wood - increased from 3 to 8 mins because server crashes were timing out and getting lost
				// NOTE: Do not increase above 8.5 mins without altering watchdog scripts to match
				const double kCrashTimeOut = 8 * 60.0;

				const double kCrashSleepInterval = 1.0;
				if (!UnixCrashReporterTracker::WaitForProcWithTimeout(RunningProc, kCrashTimeOut, kCrashSleepInterval))
				{
					FPlatformProcess::TerminateProc(RunningProc);
				}

				FPlatformProcess::CloseProc(RunningProc);
			}
		}
		else
		{
			UE_LOGF(LogCore, Warning, "MakeDirectory %ls failed", *CrashInfoAbsolute);
		}
	}

#endif

	if (!bReportingNonCrash)
	{
		// remove the handler for this signal and re-raise it (which should generate the proper core dump)
		// print message to stdout directly, it may be too late for the log (doesn't seem to be printed during a crash in the thread) 
		printf("Engine crash handling finished; re-raising signal %d for the default handler. Good bye.\n", Signal);
		fflush(stdout);

		struct sigaction ResetToDefaultAction;
		FMemory::Memzero(ResetToDefaultAction);
		ResetToDefaultAction.sa_handler = SIG_DFL;
		sigfillset(&ResetToDefaultAction.sa_mask);
		sigaction(Signal, &ResetToDefaultAction, nullptr);

		raise(Signal);
	}
}

/**
 * Good enough default crash reporter.
 */
void DefaultCrashHandler(const FUnixCrashContext & Context)
{
	printf("DefaultCrashHandler: Signal=%d\n", Context.Signal);

	// Stop the heartbeat thread so that it doesn't interfere with crashreporting
	FThreadHeartBeat::Get().Stop();

	// at this point we should already be using malloc crash handler (see PlatformCrashHandler)
	const_cast<FUnixCrashContext&>(Context).CaptureStackTrace(Context.ErrorFrame);
	const_cast<FUnixCrashContext&>(Context).CaptureAllThreadContexts();
	if (GLog)
	{
		GLog->Panic();
	}
	if (GWarn)
	{
		GWarn->Flush();
	}
	if (GError)
	{
		GError->Flush();
		GError->HandleError();
	}

	return Context.GenerateCrashInfoAndLaunchReporter();
}

/** Global pointer to crash handler */
void (* GCrashHandlerPointer)(const FGenericCrashContext & Context) = NULL;

extern int32 CORE_API GMaxNumberFileMappingCache;

extern thread_local const TCHAR* GCrashErrorMessage;
extern thread_local void* GCrashErrorProgramCounter;
extern thread_local ECrashContextType GCrashErrorType;

namespace
{
	ANSICHAR AnsiInternalBuffer[64] = { 0 };

	// Taken from AndroidPlatformCrashContext.cpp, we need to avoid allocations while in the signal handler
	const ANSICHAR* ItoANSI(uint64 Val, uint64 Base)
	{
		uint64 Index = 62;
		int32 Pad = 0;
		Base = FMath::Clamp<uint64>(Base, 2, 16);
		if (Val)
		{
			for (; Val && Index; --Index, Val /= Base, --Pad)
			{
				AnsiInternalBuffer[Index] = "0123456789abcdef"[Val % Base];
			}
		}
		else
		{
			AnsiInternalBuffer[Index--] = '0';
			--Pad;
		}

		while (Pad > 0)
		{
			AnsiInternalBuffer[Index--] = '0';
			--Pad;
		}

		return &AnsiInternalBuffer[Index + 1];
	}
}

/** True system-specific crash handler that gets called first */
void PlatformCrashHandler(int32 Signal, siginfo_t* Info, void* Context)
{
	fprintf(stderr, "Signal %d caught.\n", Signal);

	// Stop the heartbeat thread
	FThreadHeartBeat::Get().Stop();

	// Switch to malloc crash.
	FPlatformMallocCrash::Get().SetAsGMalloc();

	// Once we crash we can no longer try to find cache files. This can cause a deadlock when crashing while having locked in that file cache
	GMaxNumberFileMappingCache = 0;

	ECrashContextType Type;
	TStringBuilder<128> DefaultErrorMessage;
	const TCHAR* ErrorMessage;
	void* ErrorProgramCounter;

	if (GCrashErrorMessage == nullptr)
	{
#if UE_SERVER
		// External watchers should send SIGQUIT to kill an hanged server
		if( Signal == SIGQUIT )
		{
			Type = ECrashContextType::Hang;
		}
		else
#endif
		{
			Type = ECrashContextType::Crash;
		}

		DefaultErrorMessage.Append(TEXT("Caught signal "));
		DefaultErrorMessage.Append(ItoANSI(Signal, 10));

		ANSICHAR* SignalName = strsignal(Signal);
		if (SignalName != nullptr)
		{
			DefaultErrorMessage.Append(TEXT(" "));
			DefaultErrorMessage.Append(SignalName);
		}

		if (Signal == SIGSYS)
		{
			DefaultErrorMessage.Append(TEXT(" from syscall "));
			DefaultErrorMessage.Append(ItoANSI(Info->si_syscall, 10));
		}

		ErrorMessage = *DefaultErrorMessage;
		ErrorProgramCounter = __builtin_return_address(0);
	}
	else
	{
		Type = GCrashErrorType;
		ErrorMessage = GCrashErrorMessage;
		ErrorProgramCounter = GCrashErrorProgramCounter;
	}

	// Expose OOM root cause during generic crash (eg. SIGSEGV on nullptr).
	if (Type == ECrashContextType::Crash && FPlatformMemory::bIsOOM)
	{
		Type = ECrashContextType::OutOfMemory;
	}

	FUnixCrashContext CrashContext(Type, ErrorMessage);
	CrashContext.InitFromSignal(Signal, Info, Context);
	CrashContext.FirstCrashHandlerFrame = static_cast<uint64*>(__builtin_return_address(0));
	CrashContext.ErrorFrame = ErrorProgramCounter;

	// This will ungrab cursor/keyboard and bring down any pointer barriers which will be stuck on when opening the CRC
	FPlatformMisc::UngrabAllInput();

	if (GCrashHandlerPointer)
	{
		GCrashHandlerPointer(CrashContext);
	}
	else
	{
		// call default one
		DefaultCrashHandler(CrashContext);
	}
}

void ThreadStackWalker(int32 Signal, siginfo_t* Info, void* Context)
{
	ThreadStackUserData* ThreadStackData = static_cast<ThreadStackUserData*>(Info->si_value.sival_ptr);

	if (ThreadStackData)
	{
		if (ThreadStackData->bCaptureCallStack)
		{
			// One for the pthread frame and one for siqueue
			int32 IgnoreCount = 2;
			FPlatformStackWalk::StackWalkAndDump(ThreadStackData->CallStack, ThreadStackData->CallStackSize, IgnoreCount);
		}
		else
		{
			ThreadStackData->BackTraceCount = FPlatformStackWalk::CaptureStackBackTrace(ThreadStackData->BackTrace, ThreadStackData->CallStackSize);
		}

		ThreadStackData->bDone = true;
	}
}

void FUnixPlatformMisc::SetGracefulTerminationHandler()
{
	struct sigaction Action;
	FMemory::Memzero(Action);
	Action.sa_sigaction = GracefulTerminationHandler;
	sigfillset(&Action.sa_mask);
	Action.sa_flags = SA_SIGINFO | SA_RESTART | SA_ONSTACK;
	sigaction(SIGINT, &Action, nullptr);
	sigaction(SIGTERM, &Action, nullptr);
	sigaction(SIGHUP, &Action, nullptr);	//  this should actually cause the server to just re-read configs (restart?)
}

// Stack pointer for the main thread
void *FRunnableThreadUnix::MainThreadSignalHandlerStack = nullptr;

void *FRunnableThreadUnix::AllocCrashHandlerStack()
{
	SIZE_T PageSize = FPlatformMemory::GetConstants().PageSize;
	uint64 StackBufferSize = GetCrashHandlerStackSize();

	// grab two extra pages to protect the left/right boundary of this stack
	void* Ptr = mmap(nullptr, StackBufferSize + PageSize * 2, PROT_READ | PROT_WRITE, MAP_PRIVATE | MAP_ANON, -1, 0);

	// protect the left most page, and the right most page in case of a buffer under/over flow
	mprotect(Ptr, PageSize, PROT_NONE);
	mprotect(reinterpret_cast<uint8*>(Ptr) + StackBufferSize + PageSize, PageSize, PROT_NONE);

	return reinterpret_cast<uint8*>(Ptr) + PageSize;
}

void FRunnableThreadUnix::FreeCrashHandlerStack(void *StackBuffer)
{
	if (StackBuffer)
	{
		SIZE_T PageSize = FPlatformMemory::GetConstants().PageSize;
		uint64 StackTraceSize = GetCrashHandlerStackSize();

		// disable our current altstack so the kernel is not left with a dangling pointer, so we can then free the memory
		stack_t CurrentSignalHandlerStack;
		FMemory::Memzero(CurrentSignalHandlerStack);
		CurrentSignalHandlerStack.ss_flags = SS_DISABLE;
		sigaltstack(&CurrentSignalHandlerStack, nullptr);

		// we added an extra PageSize when allocating in ::AllocCrashHandlerStack, lets return back to the start here
		munmap(reinterpret_cast<uint8*>(StackBuffer) - PageSize, GetCrashHandlerStackSize() + PageSize * 2);
	}
}

// Defined in UnixPlatformMemory, set via -crashhandlerstacksize command line.
extern uint64 GCrashHandlerStackSize;

uint64 FRunnableThreadUnix::GetCrashHandlerStackSize()
{
	if (GCrashHandlerStackSize == 0)
	{
		GCrashHandlerStackSize = EConstants::CrashHandlerStackSize;
	}
	else if (GCrashHandlerStackSize < EConstants::CrashHandlerStackSizeMin)
	{
		GCrashHandlerStackSize = EConstants::CrashHandlerStackSizeMin;
	}

	check(IsAligned(GCrashHandlerStackSize, FPlatformMemory::GetConstants().PageSize));

	return GCrashHandlerStackSize;
}

// Defined in UnixPlatformMemory.cpp. Allows settings a specific signal to maintain its default handler rather then ignoring it
extern int32 GSignalToDefault;

void FUnixPlatformMisc::SetCrashHandler(void (* CrashHandler)(const FGenericCrashContext & Context))
{
	GCrashHandlerPointer = CrashHandler;

	// This table lists all signals that we handle. 0 is not a valid signal, it is used as a separator: everything 
	// before is considered a crash and handled by the crash handler; everything above it is handled elsewhere 
	// and also omitted from setting to ignore
	int HandledSignals[] = 
	{
		// signals we consider crashes
		SIGQUIT,
		SIGABRT,
		SIGILL, 
		SIGFPE, 
		SIGBUS, 
		SIGSEGV, 
		SIGSYS,
		SIGTRAP,
		0,	// marks the end of crash signals
		SIGINT,
		SIGTERM,
		SIGHUP,
		SIGCHLD
	};

	struct sigaction Action;
	FMemory::Memzero(Action);
	sigfillset(&Action.sa_mask);
	Action.sa_flags = SA_SIGINFO | SA_RESTART | SA_ONSTACK;
	Action.sa_sigaction = PlatformCrashHandler;

	// install this handler for all the "crash" signals
	for (int Signal : HandledSignals)
	{
		if (!Signal)
		{
			// hit the end of crash signals, the rest is already handled elsewhere
			break;
		}
		sigaction(Signal, &Action, nullptr);
	}

	// reinitialize the structure, since assigning to both sa_handler and sa_sigacton is ill-advised
	FMemory::Memzero(Action);
	sigfillset(&Action.sa_mask);
	Action.sa_flags = SA_SIGINFO | SA_RESTART | SA_ONSTACK;
	Action.sa_handler = SIG_IGN;

	// Set all the signals except ones we know we are handling to be ignored
	// Exempt realtime signals as well as they are used by third party libs and VTune
	for (int Signal = 1; Signal < SIGRTMIN; ++Signal)
	{
		bool bSignalShouldBeIgnored = true;
		for (int HandledSignal : HandledSignals)
		{
			if (Signal == HandledSignal)
			{
				bSignalShouldBeIgnored = false;
				break;
			}
		}

		if (GSignalToDefault && Signal == GSignalToDefault)
		{
			bSignalShouldBeIgnored = false;
		}

		if (bSignalShouldBeIgnored)
		{
			sigaction(Signal, &Action, nullptr);
		}
	}

	struct sigaction ActionForThread;
	FMemory::Memzero(ActionForThread);
	sigfillset(&ActionForThread.sa_mask);
	ActionForThread.sa_flags = SA_SIGINFO | SA_RESTART | SA_ONSTACK;
	ActionForThread.sa_sigaction = ThreadStackWalker;
	sigaction(THREAD_CALLSTACK_GENERATOR, &ActionForThread, nullptr);

	checkf(IsInGameThread(), TEXT("Crash handler for the game thread should be set from the game thread only."));

	if (!FRunnableThreadUnix::MainThreadSignalHandlerStack)
	{
		FRunnableThreadUnix::MainThreadSignalHandlerStack = FRunnableThreadUnix::AllocCrashHandlerStack();
	}

	FRunnableThreadUnix::SetupSignalHandlerStack(FRunnableThreadUnix::MainThreadSignalHandlerStack, FRunnableThreadUnix::GetCrashHandlerStackSize(), nullptr);
}
