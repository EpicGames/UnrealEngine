// Copyright Epic Games, Inc. All Rights Reserved.

#include "Unix/UnixPlatformFile.h"
#include "GenericPlatform/GenericPlatformProcess.h"
#include "HAL/PlatformTime.h"
#include "HAL/PlatformOutputDevices.h"
#include "Containers/StringConv.h"
#include "Containers/LruCache.h"
#include "Logging/LogMacros.h"
#include "Misc/Paths.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Async/MappedFileHandle.h"
#include "AutoRTFM.h"
#include <sys/file.h>
#include <sys/mman.h>

#include "HAL/PlatformFileCommon.h"
#include "HAL/PlatformFileManager.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "ProfilingDebugging/CountersTrace.h"

DEFINE_LOG_CATEGORY_STATIC(LogUnixPlatformFile, Log, All);

#ifndef UNIX_PLATFORM_FILE_SPEEDUP_FILE_OPERATIONS
#define UNIX_PLATFORM_FILE_SPEEDUP_FILE_OPERATIONS ((!WITH_EDITOR && !IS_PROGRAM) || !PLATFORM_LINUX) 
#endif

// make an FTimeSpan object that represents the "epoch" for time_t (from a stat struct)
const FDateTime UnixEpoch(1970, 1, 1);

class FFileHandleUnix;

namespace
{
	FFileStatData UnixStatToUEFileData(struct stat& FileInfo)
	{
		const bool bIsDirectory = S_ISDIR(FileInfo.st_mode);

		int64 FileSize = -1;
		if (!bIsDirectory)
		{
			FileSize = FileInfo.st_size;
		}

		return FFileStatData(
			UnixEpoch + FTimespan::FromSeconds(static_cast<double>(FileInfo.st_ctime)),
			UnixEpoch + FTimespan::FromSeconds(static_cast<double>(FileInfo.st_atime)),
			UnixEpoch + FTimespan::FromSeconds(static_cast<double>(FileInfo.st_mtime)),
			FileSize,
			bIsDirectory,
			!(FileInfo.st_mode & S_IWUSR)
		);
	}

	// Wrapper around UE_LOG to avoid infinite logging if the act of logging causes an error
	#define UE_LOG_UNIX_FILE(Verbosity, Format, ...) \
	{ \
		if (!bLoggingError) \
		{ \
			bLoggingError = true;  \
			UE_LOG(LogUnixPlatformFile, Verbosity, Format, ##__VA_ARGS__); \
			bLoggingError = false; \
		} \
	}

	// Wrapper around UE_LOG to avoid infinite logging if the act of logging causes an error
	#define UE_LOGF_UNIX_FILE(Verbosity, Format, ...) \
	{ \
		if (!bLoggingError) \
		{ \
			bLoggingError = true;  \
			UE_LOGF(LogUnixPlatformFile, Verbosity, Format, ##__VA_ARGS__); \
			bLoggingError = false; \
		} \
	}
}

/**
 * Unix file handle implementation which limits number of open files per thread. This
 * is to prevent running out of system file handles. Should not be neccessary when
 * using pak file (e.g., SHIPPING?) so not particularly optimized. Only manages
 * files which are opened READ_ONLY.
 */

/**
 * Unix version of the file handle registry
 */
class FUnixFileRegistry : public FFileHandleRegistry
{
public:
	FUnixFileRegistry()
		: FFileHandleRegistry(200)
	{
	}

protected:
	virtual FRegisteredFileHandle* PlatformInitialOpenFile(const TCHAR* Filename) override;
	virtual bool PlatformReopenFile(FRegisteredFileHandle* Handle) override;
	virtual void PlatformCloseFile(FRegisteredFileHandle* Handle) override;
};
static FUnixFileRegistry GFileRegistry;

/** 
 * Unix file handle implementation
 */
class CORE_API FFileHandleUnix : public FRegisteredFileHandle
{
	// https://man7.org/linux/man-pages/man2/write.2.html
	// On Linux, write() (and similar system calls) will transfer at most 0x7ffff000 (2,147,479,552) bytes,
	// returning the number of bytes actually transferred.  (This is true on both 32-bit and 64-bit systems.)
	enum {READWRITE_SIZE = 0x7ffff000};

	FORCEINLINE bool IsValid()
	{
		return FileHandle != -1;
	}

public:
	FFileHandleUnix(int32 InFileHandle, const TCHAR* InFilename, bool InFileOpenAsWrite)
		: FileHandle(InFileHandle)
		, Filename(InFilename)
		, FileOffset(0)
		, FileSize(0)
		, FileOpenAsWrite(InFileOpenAsWrite)
	{
		check(FileHandle > -1);
		check(Filename.Len() > 0);

		// Only files opened for read will be managed
		if (!FileOpenAsWrite)
		{
			struct stat FileInfo;
			fstat(FileHandle, &FileInfo);
			FileSize = FileInfo.st_size;
		}
	}

	virtual ~FFileHandleUnix()
	{
		if (FileOpenAsWrite)
		{
			close(FileHandle);
		}
		else
		{
			// only track registry for read files
			GFileRegistry.UnTrackAndCloseFile(this);
		}
		FileHandle = -1;
	}

	virtual int64 Tell() override
	{
		if (!FileOpenAsWrite)
		{
			return FileOffset;
		}
		else
		{
			check(IsValid());
			return lseek(FileHandle, 0, SEEK_CUR);
		}
	}

	virtual bool Seek(int64 NewPosition) override
	{
		check(NewPosition >= 0);

		// Avoid allowing for a negative NewPosition as this will set FileOffset which is returned in Tell blindly
		if (NewPosition < 0)
		{
			return false;
		}

		if (!FileOpenAsWrite)
		{
			FileOffset = NewPosition >= FileSize ? FileSize - 1 : NewPosition;
			return true;
		}
		else
		{
			check(IsValid());
			return lseek(FileHandle, NewPosition, SEEK_SET) != -1;
		}
	}

	virtual bool SeekFromEnd(int64 NewPositionRelativeToEnd = 0) override
	{
		check(NewPositionRelativeToEnd <= 0);

		// Avoid allowing a relative position to set less then the size of the file
		// lseek handles this negative but we return FileOffset blindly which could be used incorrectly
		if (NewPositionRelativeToEnd < (-FileSize))
		{
			return false;
		}

		if (!FileOpenAsWrite)
		{
			FileOffset = (NewPositionRelativeToEnd >= FileSize) ? 0 : (FileSize + NewPositionRelativeToEnd - 1);
			return true;
		}
		else
		{
			check(IsValid());
			return lseek(FileHandle, NewPositionRelativeToEnd, SEEK_END) != -1;
		}
	}

	virtual bool Read(uint8* Destination, int64 BytesToRead) override
	{
		check(IsValid());
		if (!FileOpenAsWrite)
		{
			// Handle virtual file handles (only in read mode, write mode doesn't use the file handle registry)
			FFileHandleRegistryReadTracker TrackRead(GFileRegistry, *this, true);
			if (!TrackRead.IsValid() || FileOffset >= FileSize)
			{
				return false;
			}

			FScopedDiskUtilizationTracker Tracker(BytesToRead, FileOffset);

			// seek to the offset on seek? this matches console behavior more closely
			if (lseek(FileHandle, FileOffset, SEEK_SET) == -1)
			{
				return false;
			}
			int64 BytesRead = ReadInternal(Destination, BytesToRead);
			FileOffset += BytesRead;
			return BytesRead == BytesToRead || FileOffset == FileSize;
		}
		else // FileOffset is invalid in 'read/write' mode, i.e. not updated by Write(), Seek(), SeekFronEnd(). Read from the current location.
		{
			int64 Pos = Tell();
			FScopedDiskUtilizationTracker Tracker(BytesToRead, Pos);
			int64 BytesRead = ReadInternal(Destination, BytesToRead);
			if (BytesRead != BytesToRead)
			{
				// If we requested more than was in the file return true if we read all the remaining content
				return BytesRead > 0 && (Pos + BytesRead) == Size();
			}
			return true;
		}
	}

	virtual bool ReadAt(uint8* Destination, int64 BytesToRead, int64 Offset) override
	{
		if (BytesToRead < 0 || Offset < 0)
		{
			return false;
		}

		if (BytesToRead == 0)
		{
			return true;
		}

		FFileHandleRegistryReadTracker TrackRead(GFileRegistry, *this, true);
		if (!FileOpenAsWrite && !TrackRead.IsValid())
		{
			return false;
		}

		do
		{
			size_t BytesToRead32 = static_cast<size_t>(FMath::Min<int64>(READWRITE_SIZE, BytesToRead));
			ssize_t BytesRead = pread(FileHandle, Destination, BytesToRead32, Offset);

			if (BytesRead == -1 && errno == EFAULT)
			{
				// Hacky workaround @see ReadInternal
				void* TempDest = FMemory::Malloc(BytesToRead32);
				if (TempDest)
				{
					BytesRead = pread(FileHandle, TempDest, BytesToRead32, Offset);
					if (BytesRead > 0 && BytesRead <= BytesToRead32)
					{
						FMemory::Memcpy(Destination, TempDest, BytesRead);
					}
					FMemory::Free(TempDest);
				}
			}

			if (BytesRead == -1 && errno == EINTR)
			{
				continue;
			}
			if (BytesRead <= 0)
			{
				return false;
			}

			Offset += BytesRead;
			BytesToRead -= BytesRead;
			Destination += BytesRead;

		} while (BytesToRead > 0);

		return true;
	}

	virtual bool Write(const uint8* Source, int64 BytesToWrite) override
	{
		check(IsValid());
		check(FileOpenAsWrite);
		while (BytesToWrite)
		{
			check(BytesToWrite >= 0);
			int64 ThisSize = FMath::Min<int64>(READWRITE_SIZE, BytesToWrite);
			check(Source);
			int64 WrittenSize = write(FileHandle, Source, ThisSize);
			if (WrittenSize == -1)
			{
				return false;
			}
			Source += WrittenSize;
			BytesToWrite -= WrittenSize;
		}
		return true;
	}

	virtual bool Flush(const bool bFullFlush = false) override
	{
		check(IsValid());
		return bFullFlush
			? fsync(FileHandle) == 0
			: fdatasync(FileHandle) == 0;
	}

	virtual bool Truncate(int64 NewSize) override
	{
		check(IsValid());
		
		int Result = 0;
		do { Result = ftruncate(FileHandle, NewSize); } while (Result < 0 && errno == EINTR);
		return Result == 0;
	}

	virtual int64 Size() override
	{
		if (!FileOpenAsWrite)
		{
			return FileSize;
		}
		else
		{
			struct stat FileInfo;
			fstat(FileHandle, &FileInfo);
			return FileInfo.st_size;
		}
	}

	
private:

	/** This function exists because we cannot read more than SSIZE_MAX at once */
	int64 ReadInternal(uint8* Destination, int64 BytesToRead)
	{
		check(IsValid());
		int64 BytesRead = 0;

		if (BytesToRead < 0)
		{
			return 0;
		}

		while (BytesToRead)
		{
			check(BytesToRead >= 0);
			int64 ThisSize = FMath::Min<int64>(READWRITE_SIZE, BytesToRead);
			check(Destination);
			int64 ThisRead = read(FileHandle, Destination, ThisSize);
			
			if (ThisRead == -1 && errno == EFAULT)
			{
				// hacky workaround for UE-69018
				void* TempDest = FMemory::Malloc(ThisSize);
				if (TempDest)
				{
					ThisRead = read(FileHandle, TempDest, ThisSize);
					if (ThisRead > 0 && ThisRead <= ThisSize)
					{
						FMemory::Memcpy(Destination, TempDest, ThisRead);
					}
					FMemory::Free(TempDest);
				}
			}
			
			if (ThisRead == -1 && errno == EINTR)   // need to retry
			{
				continue;
			}			
			
			if (ThisRead == -1)  // some other error just bail
			{
				return BytesRead;
			}

			BytesRead += ThisRead;
			if (ThisRead != ThisSize)
			{
				return BytesRead;
			}
			Destination += ThisSize;
			BytesToRead -= ThisSize;
		}
		return BytesRead;
	}

	// Holds the internal file handle.
	int32 FileHandle;

	// Holds the name of the file that this handle represents. Kept around for possible reopen of file.
	FString Filename;

	// Most recent valid slot index for this handle; >=0 for handles which are managed.
	int32 HandleSlot;

	// Current file offset; valid if a managed handle.
	int64 FileOffset;

	// Cached file size; valid if a managed handle.
	int64 FileSize;

	// track if file is open for write
	bool FileOpenAsWrite;

	friend class FUnixFileRegistry;
};

extern int32 CORE_API GMaxNumberFileMappingCache;

namespace
{
	double MaxInvalidCacheTime = 0.5; // 500ms

	struct FileEntry
	{
		FString File;
		bool bInvalid = false;
		double CacheTime = 0.0f;

		bool IsInvalid() const
		{
			double Current = FPlatformTime::Seconds();
			return bInvalid && Current - CacheTime <= MaxInvalidCacheTime;
		}
	};

	struct FileMapCache
	{
		virtual const FileEntry* Find(const FString& Key) = 0;
		virtual void AddEntry(const FString& Key, const FString& Elem) = 0;
		virtual void Invalidate(const FString& Key) = 0;
		virtual void Lock() = 0;
		virtual void Unlock() = 0;
	};

	class FileMapCacheDummy : public FileMapCache
	{
	public:
		const FileEntry* Find(const FString& Key) override
		{
			return nullptr;
		}

		void AddEntry(const FString& Key, const FString& Elem) override
		{ }

		void Invalidate(const FString& Key) override
		{ }

		void Lock() override
		{ }

		void Unlock() override
		{ }
	};

	class FileMapCacheDefault : public FileMapCache
	{
	public:
		FileMapCacheDefault()
			: Cache(GMaxNumberFileMappingCache)
		{
		}

		const FileEntry* Find(const FString& Key) override
		{
			return Cache.FindAndTouch(Key);
		}

		void AddEntry(const FString& Key, const FString& Elem) override
		{
			Cache.Add(Key, {Elem, Elem.IsEmpty(), FPlatformTime::Seconds()});
		}

		void Invalidate(const FString& Key) override
		{
			Cache.Remove(Key);
		}

		void Lock() override
		{
			Mutex.Lock();
		}

		void Unlock() override
		{
			Mutex.Unlock();
		}

	private:
		FCriticalSection Mutex;
		TLruCache<FString, FileEntry> Cache;
	};

	FileMapCache& GetFileMapCache()
	{
		static FileMapCacheDefault DefaultCache;
		static FileMapCacheDummy DummyCache;

		return GMaxNumberFileMappingCache > 0 ? *static_cast<FileMapCache*>(&DefaultCache) : *static_cast<FileMapCache*>(&DummyCache);
	}
}

/**
 * A class to handle case insensitive file opening. This is a band-aid, non-performant approach,
 * without any caching.
 */

 TRACE_DECLARE_INT_COUNTER(FUnixFileMapper_UnnecessaryFileRemapping, TEXT("FUnixFileMapper/Unnecessary File Remappings"));
 TRACE_DECLARE_INT_COUNTER(FUnixFileMapper_NecessaryFileRemapping, TEXT("FUnixFileMapper/Necessary File Remappings"));

class FUnixFileMapper
{

public:

	FUnixFileMapper()
	{
	}

	FString GetPathComponent(const FString & Filename, int NumPathComponent)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FUnixFileMapper::GetPathComponent);

		// skip over empty part
		int StartPosition = (Filename[0] == TEXT('/')) ? 1 : 0;
		
		for (int ComponentIdx = 0; ComponentIdx < NumPathComponent; ++ComponentIdx)
		{
			int FoundAtIndex = Filename.Find(TEXT("/"), ESearchCase::CaseSensitive,
											 ESearchDir::FromStart, StartPosition);
			
			if (FoundAtIndex == INDEX_NONE)
			{
				checkf(false, TEXT("Asked to get %d-th path component, but filename '%s' doesn't have that many!"), 
						 NumPathComponent, *Filename);
				break;
			}
			
			StartPosition = FoundAtIndex + 1;	// skip the '/' itself
		}

		// now return the 
		int NextSlash = Filename.Find(TEXT("/"), ESearchCase::CaseSensitive,
										ESearchDir::FromStart, StartPosition);
		if (NextSlash == INDEX_NONE)
		{
			// just return the rest of the string
			return Filename.RightChop(StartPosition);
		}
		else if (NextSlash == StartPosition)
		{
			return TEXT("");	// encountered an invalid path like /foo/bar//baz
		}
		
		return Filename.Mid(StartPosition, NextSlash - StartPosition);
	}

	int32 CountPathComponents(const FString & Filename)
	{
		if (Filename.Len() == 0)
		{
			return 0;
		}

		// if the first character is not a separator, it's part of a distinct component
		int NumComponents = (Filename[0] != TEXT('/')) ? 1 : 0;
		for (const auto & Char : Filename)
		{
			if (Char == TEXT('/'))
			{
				++NumComponents;
			}
		}

		// cannot be 0 components if path is non-empty
		return FMath::Max(NumComponents, 1);
	}

	/**
	 * Tries to recursively find (using case-insensitive comparison) and open the file.
	 * The first file found will be opened.
	 * 
	 * @param Filename Original file path as requested (absolute)
	 * @param PathComponentToLookFor Part of path we are currently trying to find.
	 * @param MaxPathComponents Maximum number of path components (directories), i.e. how deep the path is.
	 * @param ConstructedPath The real (absolute) path that we have found so far
	 * 
	 * @return a handle opened with open()
	 */
	bool MapFileRecursively(const FString & Filename, int PathComponentToLookFor, int MaxPathComponents, FString & ConstructedPath)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FUnixFileMapper::MapFileRecursively);

		// get the directory without the last path component
		FString BaseDir = ConstructedPath;

		// get the path component to compare
		FString PathComponent = GetPathComponent(Filename, PathComponentToLookFor);
		FString PathComponentLower = PathComponent.ToLower();

		bool bFound = false;

		// see if we can open this (we should)
		DIR* DirHandle = opendir(TCHAR_TO_UTF8(*BaseDir));
		if (DirHandle)
		{
			struct dirent *Entry;
			while ((Entry = readdir(DirHandle)) != nullptr)
			{
				FString DirEntry = UTF8_TO_TCHAR(Entry->d_name);
				if (DirEntry.ToLower() == PathComponentLower)
				{
					if (PathComponentToLookFor < MaxPathComponents - 1)
					{
						// make sure this is a directory
						bool bIsDirectory = Entry->d_type == DT_DIR;
						if(Entry->d_type == DT_UNKNOWN || Entry->d_type == DT_LNK)
						{
							struct stat StatInfo;
							if(stat(TCHAR_TO_UTF8(*(BaseDir / Entry->d_name)), &StatInfo) == 0)
							{
								bIsDirectory = S_ISDIR(StatInfo.st_mode);
							}
						}

						if (bIsDirectory)
						{
							// recurse with the new filename
							FString NewConstructedPath = ConstructedPath;
							NewConstructedPath /= DirEntry;

							bFound = MapFileRecursively(Filename, PathComponentToLookFor + 1, MaxPathComponents, NewConstructedPath);
							if (bFound)
							{
								ConstructedPath = NewConstructedPath;
								break;
							}
						}
					}
					else
					{
						// last level, try opening directly
						FString ConstructedFilename = ConstructedPath;
						ConstructedFilename /= DirEntry;

						struct stat StatInfo;
						bFound = (stat(TCHAR_TO_UTF8(*ConstructedFilename), &StatInfo) == 0);
						if (bFound)
						{
							ConstructedPath = ConstructedFilename;
							break;
						}
					}
				}
			}
			closedir(DirHandle);
		}
		
		return bFound;
	}

	/**
	 * Tries to map a filename (one with a possibly wrong case) to one that exists.
	 * 
	 * @param PossiblyWrongFilename absolute filename (that has possibly a wrong case)
	 * @param ExistingFilename filename that exists (only valid to use if the function returned success).
	 */
	bool MapCaseInsensitiveFile(const FString & PossiblyWrongFilename, FString & ExistingFilename)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FUnixFileMapper::MapCaseInsensitiveFile);

		// Have to be careful logging in this function since this is called for setting up the log file

		// We can get some "absolute" filenames like "D:/Blah/" here (e.g. non-Linux paths to source files embedded in assets).
		// In that case, fail silently.
		if (PossiblyWrongFilename.IsEmpty() || PossiblyWrongFilename[0] != TEXT('/'))
		{
			return false;
		}

		bool bFound = false;
		ExistingFilename = AutoRTFM::Open([this, &PossiblyWrongFilename, &bFound]
		{
			// Try the filename as given first.
			struct stat StatInfo;
			bFound = stat(TCHAR_TO_UTF8(*PossiblyWrongFilename), &StatInfo) == 0;

			if (bFound)
			{
				return PossiblyWrongFilename;
			}

			// Next, check in the cache.
			FileMapCache& Cache = GetFileMapCache();
			UE::TScopeLock Lock(Cache);
			const FileEntry* Entry = Cache.Find(PossiblyWrongFilename);

			if (Entry != nullptr)
			{
				if (Entry->IsInvalid())
				{
					bFound = false;
					return FString();
				}
				else
				{
					bFound = true;
					return Entry->File;
				}
			}

			// We haven't seen this path before. Perform a case-insensitive search from /
			int MaxPathComponents = CountPathComponents(PossiblyWrongFilename);
			if (MaxPathComponents == 0)
			{
				// Non-empty paths should always have at least one component; we don't expect this to happen.
				return FString();
			}

			FString FoundFilename("/");	// Start from the root.
			bFound = MapFileRecursively(PossiblyWrongFilename, 0, MaxPathComponents, FoundFilename);
			if (bFound)
			{
				Cache.AddEntry(PossiblyWrongFilename, FoundFilename);
				return FoundFilename;
			}
			else
			{
				// Cache a failed to find entry. We'll look again if the call in is greater then MaxInvalidCacheTime from this cache point.
				Cache.AddEntry(PossiblyWrongFilename, "");
				return FString();
			}
		});

		if (bFound && !PossiblyWrongFilename.Equals(ExistingFilename, ESearchCase::CaseSensitive) && !ExistingFilename.IsEmpty())
		{
			// Can't just log directly here.  Use LowLevelOutputDebugString if logging isn't hooked up.
			if (GLog && FPlatformOutputDevices::GetLog() && GLog->IsRedirectingTo(FPlatformOutputDevices::GetLog()))
			{
				UE_LOGF(LogUnixPlatformFile, Log, "FUnixFileMapper::MapCaseInsensitiveFile: Lookup:%ls Found:%ls\n", *PossiblyWrongFilename, *ExistingFilename);
			}
			else
			{
				FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FUnixFileMapper::MapCaseInsensitiveFile: Lookup:%s Found:%s\n"), *PossiblyWrongFilename, *ExistingFilename);
			}
			TRACE_COUNTER_ADD(FUnixFileMapper_NecessaryFileRemapping, 1);
		}
		else
		{
			TRACE_COUNTER_ADD(FUnixFileMapper_UnnecessaryFileRemapping, 1);
		}
		return bFound;
	}

	/**
	 * Opens a file for reading, disregarding the case.
	 * 
	 * @param Filename absolute filename
	 * @param MappedToFilename absolute filename that we mapped the Filename to (always filled out on success, even if the same as Filename)
	 */
	int32 OpenCaseInsensitiveRead(const FString & Filename, FString & MappedToFilename)
	{
		// We can get some "absolute" filenames like "D:/Blah/" here (e.g. non-Linux paths to source files embedded in assets).
		// In that case, fail silently.
		if (Filename.IsEmpty() || Filename[0] != TEXT('/'))
		{
			return -1;
		}

		// try opening right away
		int32 Handle = open(TCHAR_TO_UTF8(*Filename), O_RDONLY | O_CLOEXEC);
		const int ErrNo = errno;
		if (Handle != -1)
		{
			MappedToFilename = Filename;
		}
		else
		{
			// log non-standard errors only
			if (ENOENT != ErrNo)
			{
				UE_LOGF(LogUnixPlatformFile, Warning, "open('%ls', O_RDONLY | O_CLOEXEC) failed: errno=%d (%ls)", *Filename, ErrNo, UTF8_TO_TCHAR(strerror(ErrNo)));
			}
			else
			{
#if (UE_GAME || UE_SERVER)
				// Games (including clients) and servers have no business traversing the filesystem when reading from the pak files - make sure the paths are correct!
				static bool bReadingFromPakFiles = FPlatformFileManager::Get().FindPlatformFile(TEXT("PakFile")) != nullptr;
				if (LIKELY(bReadingFromPakFiles))
				{
					return -1;
				}
#endif
				TRACE_CPUPROFILER_EVENT_SCOPE(FUnixFileMapper::OpenCaseInsensitiveRead);
				// perform a case-insensitive search
				// make sure we get the absolute filename
				checkf(Filename[0] == TEXT('/'), TEXT("Filename '%s' given to OpenCaseInsensitiveRead is not absolute!"), *Filename);

				int MaxPathComponents = CountPathComponents(Filename);
				if (MaxPathComponents > 0)
				{
					FString FoundFilename("/");	// start with root
					if (MapFileRecursively(Filename, 0, MaxPathComponents, FoundFilename))
					{
						Handle = open(TCHAR_TO_UTF8(*FoundFilename), O_RDONLY | O_CLOEXEC);
						if (Handle != -1)
						{
							MappedToFilename = FoundFilename;
							if (!Filename.Equals(MappedToFilename, ESearchCase::CaseSensitive))
							{
								UE_LOGF(LogUnixPlatformFile, Log, "FUnixFileMapper::OpenCaseInsensitiveRead: '%ls' to '%ls'", *Filename, *MappedToFilename);
								TRACE_COUNTER_ADD(FUnixFileMapper_NecessaryFileRemapping, 1);
							}
						}
					}
					else
					{
						TRACE_COUNTER_ADD(FUnixFileMapper_UnnecessaryFileRemapping, 1);
					}
				}
			}
		}
		return Handle;
	}
};

FUnixFileMapper GCaseInsensMapper;

class FUnixMappedFileRegion final : public IMappedFileRegion
{
public:
	class FUnixMappedFileHandle* Parent;
	const uint8* AlignedPtr;
	uint64 AlignedSize;
	FUnixMappedFileRegion(const uint8* InMappedPtr, const uint8* InAlignedPtr, size_t InMappedSize, uint64 InAlignedSize, const FString& InDebugFilename, size_t InDebugOffsetIntoFile, FUnixMappedFileHandle* InParent)
		: IMappedFileRegion(InMappedPtr, InMappedSize, InDebugFilename, InDebugOffsetIntoFile)
		, Parent(InParent)
		, AlignedPtr(InAlignedPtr)
		, AlignedSize(InAlignedSize)
	{
	}

	virtual ~FUnixMappedFileRegion();
};

static SIZE_T FileMappingAlignment = FPlatformMemory::GetConstants().PageSize;

class FUnixMappedFileHandle final : public IMappedFileHandle
{
public:
	FUnixMappedFileHandle(int InFileHandle, int64 FileSize, const FString& InFilename)
		: IMappedFileHandle(FileSize)
		, MappedPtr(nullptr)
		, Filename(InFilename)
		, NumOutstandingRegions(0)
		, FileHandle(InFileHandle)
	{
	}

	virtual ~FUnixMappedFileHandle() override
	{
		// can't delete the file before you delete all outstanding regions
		UE_CLOG(
				NumOutstandingRegions.load(std::memory_order_relaxed),
				LogHAL,
#if UE_BUILD_SHIPPING
				Error,
#else
				Fatal,
#endif
				TEXT("Cleaning mapped file with alive mapped regions: %s"),
				*Filename);
		close(FileHandle);
	}

	virtual IMappedFileRegion* MapRegion(int64 Offset = 0, int64 BytesToMap = MAX_int64, FFileMappingFlags Flags = EMappedFileFlags::ENone) override
	{
		LLM_PLATFORM_SCOPE(ELLMTag::PlatformMMIO);
		check(Offset < GetFileSize()); // don't map zero bytes and don't map off the end of the file
		BytesToMap = FMath::Min<int64>(BytesToMap, GetFileSize() - Offset);
		check(BytesToMap > 0); // don't map zero bytes

		const int64 AlignedOffset = AlignDown(Offset, FileMappingAlignment);
		//File mapping can extend beyond file size. It's OK, kernel will just fill any leftover page data with zeros
		const int64 AlignedSize = Align(BytesToMap + Offset - AlignedOffset, FileMappingAlignment);

		int Protection = PROT_READ;
		int InternalFlags = (EnumHasAnyFlags(Flags.Flags, EMappedFileFlags::EPreloadHint)) ? MAP_POPULATE : 0;		
		if (EnumHasAnyFlags(Flags.Flags, EMappedFileFlags::EFileWritable))
		{
			Protection |= PROT_WRITE;
			InternalFlags |= MAP_SHARED;
		}
		else
		{
			InternalFlags |= MAP_PRIVATE;
		}

		const uint8* AlignedMapPtr = static_cast<const uint8*>(mmap(nullptr, AlignedSize, Protection, InternalFlags, FileHandle, AlignedOffset));
		if (AlignedMapPtr == MAP_FAILED || AlignedMapPtr == nullptr)
		{
			UE_LOGF(LogUnixPlatformFile, Warning, "Failed to map memory %ls, error is %d", *Filename, errno);
			return nullptr;
		}
		LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelAlloc(ELLMTracker::Platform, AlignedMapPtr, AlignedSize));

		// create a mapping for this range
		const uint8* MapPtr = AlignedMapPtr + Offset - AlignedOffset;
		FUnixMappedFileRegion* Result = new FUnixMappedFileRegion(MapPtr, AlignedMapPtr, BytesToMap, AlignedSize, Filename, Offset, this);
		NumOutstandingRegions.fetch_add(1, std::memory_order_relaxed);
		return Result;
	}

	void UnMap(const FUnixMappedFileRegion* Region)
	{
		LLM_PLATFORM_SCOPE(ELLMTag::PlatformMMIO);
		int32 OldNumOutstandingRegions = NumOutstandingRegions.fetch_sub(1, std::memory_order_relaxed);
		check(OldNumOutstandingRegions > 0);

		LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelFree(ELLMTracker::Platform, Region->AlignedPtr));
		const int Res = munmap(const_cast<uint8*>(Region->AlignedPtr), Region->AlignedSize);
		checkf(Res == 0, TEXT("Failed to unmap, error is %d, errno is %d [params: %llx, %lld]"), Res, errno, MappedPtr, GetFileSize());
	}

private:
	const uint8* MappedPtr;
	FString Filename;
	std::atomic<int32> NumOutstandingRegions;
	int FileHandle;
};

FUnixMappedFileRegion::~FUnixMappedFileRegion()
{
	Parent->UnMap(this);
}

/**
 * Unix File I/O implementation
**/
FString FUnixPlatformFile::NormalizeFilename(const TCHAR* Filename, bool bIsForWriting)
{
	FString Result(Filename);

	// If we are already absolute return
	if (!FPaths::IsRelative(Result))
	{
		return Result;
	}

	FPaths::NormalizeFilename(Result);
	return FPaths::ConvertRelativePathToFull(Result);
}

FString FUnixPlatformFile::NormalizeDirectory(const TCHAR* Directory, bool bIsForWriting)
{
	// Both these functions do the same thing
	return NormalizeFilename(Directory, bIsForWriting);
}

bool FUnixPlatformFile::FileExists(const TCHAR* Filename)
{
	FString CaseSensitiveFilename;
	FString NormalizedFilename = NormalizeFilename(Filename, false);
#if !UNIX_PLATFORM_FILE_SPEEDUP_FILE_OPERATIONS
	if (!GCaseInsensMapper.MapCaseInsensitiveFile(NormalizedFilename, CaseSensitiveFilename))
	{
		// could not find the file
		return false;
	}
#else
	CaseSensitiveFilename = NormalizedFilename;
#endif

	struct stat FileInfo;
	if(stat(TCHAR_TO_UTF8(*CaseSensitiveFilename), &FileInfo) != -1)
	{
		return S_ISREG(FileInfo.st_mode);
	}
	return false;
}

int64 FUnixPlatformFile::FileSize(const TCHAR* Filename)
{
	FString CaseSensitiveFilename;
	FString NormalizedFilename = NormalizeFilename(Filename, false);
#if !UNIX_PLATFORM_FILE_SPEEDUP_FILE_OPERATIONS
	if (!GCaseInsensMapper.MapCaseInsensitiveFile(NormalizedFilename, CaseSensitiveFilename))
	{
		// could not find the file
		return -1;
	}
#else
	CaseSensitiveFilename = NormalizedFilename;
#endif

	struct stat FileInfo;
	FileInfo.st_size = -1;
	if (stat(TCHAR_TO_UTF8(*CaseSensitiveFilename), &FileInfo) != -1)
	{
		// make sure to return -1 for directories
		if (S_ISDIR(FileInfo.st_mode))
		{
			FileInfo.st_size = -1;
		}
	}
	return FileInfo.st_size;
}

bool FUnixPlatformFile::DeleteFile(const TCHAR* Filename)
{
	FString CaseSensitiveFilename;
	FString IntendedFilename(NormalizeFilename(Filename, true));
	if (!GCaseInsensMapper.MapCaseInsensitiveFile(IntendedFilename, CaseSensitiveFilename))
	{
		// could not find the file
		return false;
	}

	{
		FileMapCache& Cache = GetFileMapCache();
		UE::TScopeLock Lock(Cache);
		Cache.Invalidate(IntendedFilename);
	}

	// removing mapped file is too dangerous
	if (IntendedFilename != CaseSensitiveFilename)
	{
		UE_LOG_UNIX_FILE(Warning, TEXT("Could not find file '%s', deleting file '%s' instead (for consistency with the rest of file ops)"), *IntendedFilename, *CaseSensitiveFilename);
	}
	return unlink(TCHAR_TO_UTF8(*CaseSensitiveFilename)) == 0;
}

bool FUnixPlatformFile::IsReadOnly(const TCHAR* Filename)
{
	FString CaseSensitiveFilename;
	if (!GCaseInsensMapper.MapCaseInsensitiveFile(NormalizeFilename(Filename, true), CaseSensitiveFilename))
	{
		// could not find the file
		return false;
	}

	// skipping checking F_OK since this is already taken care of by case mapper

	if (access(TCHAR_TO_UTF8(*CaseSensitiveFilename), W_OK) == -1)
	{
		return errno == EACCES;
	}
	return false;
}

bool FUnixPlatformFile::MoveFile(const TCHAR* To, const TCHAR* From)
{
	FString CaseSensitiveFilename;
	FString IntendedFilename = NormalizeFilename(From, true);
	if (!GCaseInsensMapper.MapCaseInsensitiveFile(IntendedFilename, CaseSensitiveFilename))
	{
		// could not find the file
		return false;
	}

	{
		FileMapCache& Cache = GetFileMapCache();
		UE::TScopeLock Lock(Cache);
		Cache.Invalidate(IntendedFilename);
	}

	int32 Result = rename(TCHAR_TO_UTF8(*CaseSensitiveFilename), TCHAR_TO_UTF8(*NormalizeFilename(To, true)));
	if (Result == -1 && errno == EXDEV)
	{
		// Copy the file if rename failed because To and From are on different file systems
		if (CopyFile(To, *CaseSensitiveFilename))
		{
			DeleteFile(*CaseSensitiveFilename);
			Result = 0;
		}
	}
	return Result != -1;
}

bool FUnixPlatformFile::SetReadOnly(const TCHAR* Filename, bool bNewReadOnlyValue)
{
	FString CaseSensitiveFilename;
	if (!GCaseInsensMapper.MapCaseInsensitiveFile(NormalizeFilename(Filename, true), CaseSensitiveFilename))
	{
		// could not find the file
		return false;
	}

	struct stat FileInfo;
	if (stat(TCHAR_TO_UTF8(*CaseSensitiveFilename), &FileInfo) != -1)
	{
		if (bNewReadOnlyValue)
		{
			FileInfo.st_mode &= ~S_IWUSR;
		}
		else
		{
			FileInfo.st_mode |= S_IWUSR;
		}
		return chmod(TCHAR_TO_UTF8(*CaseSensitiveFilename), FileInfo.st_mode) == 0;
	}
	return false;
}

FDateTime FUnixPlatformFile::GetTimeStamp(const TCHAR* Filename)
{
	FString CaseSensitiveFilename;
	if (!GCaseInsensMapper.MapCaseInsensitiveFile(NormalizeFilename(Filename, false), CaseSensitiveFilename))
	{
		// could not find the file
		return FDateTime::MinValue();
	}
	
	// get file times
	struct stat FileInfo;
	if(stat(TCHAR_TO_UTF8(*CaseSensitiveFilename), &FileInfo) == -1)
	{
		if (errno == EOVERFLOW)
		{
			// hacky workaround for files mounted on Samba (see https://bugzilla.samba.org/show_bug.cgi?id=7707)
			return FDateTime::Now();
		}
		else
		{
			return FDateTime::MinValue();
		}
	}

	// convert _stat time to FDateTime
	FTimespan TimeSinceEpoch(0, 0, FileInfo.st_mtime);
	return UnixEpoch + TimeSinceEpoch;
}

void FUnixPlatformFile::SetTimeStamp(const TCHAR* Filename, const FDateTime DateTime)
{
	FString CaseSensitiveFilename;
	if (!GCaseInsensMapper.MapCaseInsensitiveFile(NormalizeFilename(Filename, true), CaseSensitiveFilename))
	{
		// could not find the file
		return;
	}

	// get file times
	struct stat FileInfo;
	if(stat(TCHAR_TO_UTF8(*CaseSensitiveFilename), &FileInfo) == -1)
	{
		return;
	}

	// change the modification time only
	struct utimbuf Times;
	Times.actime = FileInfo.st_atime;
	Times.modtime = static_cast<__time_t>((DateTime - UnixEpoch).GetTotalSeconds());
	utime(TCHAR_TO_UTF8(*CaseSensitiveFilename), &Times);
}

FDateTime FUnixPlatformFile::GetAccessTimeStamp(const TCHAR* Filename)
{
	FString CaseSensitiveFilename;
	if (!GCaseInsensMapper.MapCaseInsensitiveFile(NormalizeFilename(Filename, true), CaseSensitiveFilename))
	{
		// could not find the file
		return FDateTime::MinValue();
	}

	// get file times
	struct stat FileInfo;
	if(stat(TCHAR_TO_UTF8(*CaseSensitiveFilename), &FileInfo) == -1)
	{
		return FDateTime::MinValue();
	}

	// convert _stat time to FDateTime
	FTimespan TimeSinceEpoch(0, 0, FileInfo.st_atime);
	return UnixEpoch + TimeSinceEpoch;
}

FString FUnixPlatformFile::GetFilenameOnDisk(const TCHAR* Filename)
{
	return Filename;
/*
	FString CaseSensitiveFilename;
	if (!GCaseInsensMapper.MapCaseInsensitiveFile(NormalizeFilename(Filename), CaseSensitiveFilename))
	{
		return Filename;
	}

	return CaseSensitiveFilename;
*/
}

ESymlinkResult FUnixPlatformFile::IsSymlink(const TCHAR* Filename)
{
	FString CaseSensitiveFilename;
	FString NormalizedFilename = NormalizeFilename(Filename, false);
#if !UNIX_PLATFORM_FILE_SPEEDUP_FILE_OPERATIONS
	if (!GCaseInsensMapper.MapCaseInsensitiveFile(NormalizedFilename, CaseSensitiveFilename))
	{
		// could not find the file
		return ESymlinkResult::NonSymlink;
	}
#else
	CaseSensitiveFilename = NormalizedFilename;
#endif

	struct stat FileInfo;
	if(stat(TCHAR_TO_UTF8(*CaseSensitiveFilename), &FileInfo) != -1 && S_ISLNK(FileInfo.st_mode))
	{
		return ESymlinkResult::Symlink;
	}
	return ESymlinkResult::NonSymlink;
}

IFileHandle* FUnixPlatformFile::OpenRead(const TCHAR* Filename, bool bAllowWrite)
{
	// let the file registry manage read files
	return GFileRegistry.InitialOpenFile(*NormalizeFilename(Filename, false));
}

IFileHandle* FUnixPlatformFile::OpenWrite(const TCHAR* Filename, bool bAppend, bool bAllowRead)
{
	int Flags = O_CREAT | O_CLOEXEC;	// prevent children from inheriting this

	if (bAllowRead)
	{
		Flags |= O_RDWR;
	}
	else
	{
		Flags |= O_WRONLY;
	}

	// We may have cached this as an invalid file, so lets just remove a newly created file from the cache
	{
		FileMapCache& Cache = GetFileMapCache();
		UE::TScopeLock Lock(Cache);
		Cache.Invalidate(FString(Filename));
	}

	// create directories if needed.
	if (!CreateDirectoriesFromPath(Filename))
	{
		return NULL;
	}

	// Caveat: cannot specify O_TRUNC in flags, as this will corrupt the file which may be "locked" by other process. We will ftruncate() it once we "lock" it
	int32 Handle = open(TCHAR_TO_UTF8(*NormalizeFilename(Filename, true)), Flags, S_IRUSR | S_IWUSR | S_IRGRP | S_IWGRP | S_IROTH);
	if (Handle != -1)
	{
		// Due to dotnet not allowing any files marked as LOCK_EX to be opened for read only or copied, this allows us to
		// to disable the locking mechanics. https://github.com/dotnet/runtime/issues/34126
		extern bool GAllowExclusiveLockOnWrite;

		// mimic Windows "exclusive write" behavior (we don't use FILE_SHARE_WRITE) by locking the file.
		// note that the (non-mandatory) "lock" will be removed by itself when the last file descriptor is close()d
		if (GAllowExclusiveLockOnWrite && (flock(Handle, LOCK_EX | LOCK_NB) == -1))
		{
			// if locked, consider operation a failure
			if (EAGAIN == errno || EWOULDBLOCK == errno)
			{
				close(Handle);
				return nullptr;
			}
			// all the other locking errors are ignored.
		}

		// truncate the file now that we locked it
		if (!bAppend)
		{
			if (ftruncate(Handle, 0) != 0)
			{
				int ErrNo = errno;
				UE_LOG_UNIX_FILE(Warning, TEXT( "ftruncate() failed for '%s': errno=%d (%s)" ),
												Filename, ErrNo, UTF8_TO_TCHAR(strerror(ErrNo)));
				close(Handle);
				return nullptr;
			}
		}

		FFileHandleUnix* FileHandleUnix = new FFileHandleUnix(Handle, *NormalizeDirectory(Filename, true), true);

		if (bAppend)
		{
			FileHandleUnix->SeekFromEnd(0);
		}
		return FileHandleUnix;
	}

	const int ErrNo = errno;
	UE_LOG_UNIX_FILE(Warning, TEXT( "open('%s', Flags=0x%08X) failed: errno=%d (%s)" ), *NormalizeFilename(Filename, true), Flags, ErrNo, UTF8_TO_TCHAR(strerror(ErrNo)));
	return nullptr;
}

FOpenMappedResult FUnixPlatformFile::OpenMappedEx(const TCHAR* Filename, EOpenReadFlags OpenOptions, int64 MaximumSize)
{
	EOpenMappedFlags ActualOpenOptions = EOpenMappedFlags::None;
	if (EnumHasAnyFlags(OpenOptions, EOpenReadFlags::AllowWrite))
	{
		ActualOpenOptions |= EOpenMappedFlags::OpenForWrite;
		ActualOpenOptions |= EOpenMappedFlags::OpenExisting;
	}

	return OpenMappedEx2(Filename, ActualOpenOptions, MaximumSize);
}

FOpenMappedResult FUnixPlatformFile::OpenMappedEx2(const TCHAR* Filename, EOpenMappedFlags OpenOptions, int64 MaximumSize)
{
	const FString NormalizedFilename = NormalizeFilename(Filename, false);

	int Flags = O_RDONLY | O_CLOEXEC;
	if (EnumHasAnyFlags(OpenOptions, EOpenMappedFlags::OpenForWrite))
	{
		Flags = O_RDWR | O_CLOEXEC;
		if (!EnumHasAnyFlags(OpenOptions, EOpenMappedFlags::OpenExisting))
		{
			Flags |= O_CREAT;
			Flags |= O_TRUNC;
		}
	}

	int32 Handle = -1;
	if (EnumHasAnyFlags(OpenOptions, EOpenMappedFlags::OpenForWrite))
	{
		Handle = open(TCHAR_TO_UTF8(*NormalizedFilename), Flags, S_IRUSR | S_IWUSR | S_IRGRP | S_IWGRP | S_IROTH);
	}
	else
	{
		Handle = open(TCHAR_TO_UTF8(*NormalizedFilename), Flags);
	}

	if (Handle == -1)
	{
		const int ErrNo = errno;
		FString ErrorStr = FString::Printf(TEXT("open('%s', Flags=0x%08X) failed: errno=%d (%s)"), *NormalizedFilename, Flags, ErrNo, UTF8_TO_TCHAR(strerror(ErrNo)));
		UE_LOG_UNIX_FILE(Warning, TEXT("%s"), *ErrorStr);

		return MakeError(MoveTemp(ErrorStr));
	}

	struct stat FileInfo;
	FileInfo.st_size = -1;
	const int StatResult = fstat(Handle, &FileInfo);
	if (StatResult == -1)
	{
		const int ErrNo = errno;
		FString ErrorStr = FString::Printf(TEXT("stat('%s', Flags=0x%08X) failed: errno=%d (%s)"), *NormalizedFilename, Flags, ErrNo, UTF8_TO_TCHAR(strerror(ErrNo)));
		UE_LOG_UNIX_FILE(Warning, TEXT("%s"), *ErrorStr);

		close(Handle);
		return MakeError(MoveTemp(ErrorStr));
	}

	if (EnumHasAnyFlags(OpenOptions, EOpenMappedFlags::OpenForWrite))
	{
		if (MaximumSize < 1)
		{
			if (EnumHasAnyFlags(OpenOptions, EOpenMappedFlags::OpenExisting))
			{
				if (FileInfo.st_size < 1)
				{
					close(Handle);
					return MakeError(TEXT("File size is 0"));
				}

				MaximumSize = FileInfo.st_size;
			}
			else
			{
				close(Handle);
				return MakeError(TEXT("No maximum size provided."));
			}
		}
		else if (MaximumSize > FileInfo.st_size)
		{
			int TruncResult = 0;
			do { TruncResult = ftruncate(Handle, MaximumSize); } while (TruncResult < 0 && errno == EINTR);

			if (TruncResult == -1)
			{
				int ErrNo = errno;
				FString ErrorStr = FString::Printf(TEXT("ftruncate() failed for '%s': errno=%d (%s)"), *NormalizedFilename, ErrNo, UTF8_TO_TCHAR(strerror(ErrNo)));
				UE_LOG_UNIX_FILE(Warning, TEXT("%s"), *ErrorStr);

				close(Handle);
				return MakeError(MoveTemp(ErrorStr));
			}
		}
	}
	else
	{
		if (FileInfo.st_size < 1)
		{
			close(Handle);
			return MakeError(TEXT("File size is 0"));
		}

		if (MaximumSize < 1 || MaximumSize > FileInfo.st_size)
		{
			MaximumSize = FileInfo.st_size;
		}
	}

	return MakeValue(new FUnixMappedFileHandle(Handle, MaximumSize, NormalizedFilename));
}

bool FUnixPlatformFile::DirectoryExists(const TCHAR* Directory)
{
	FString CaseSensitiveFilename;
	FString NormalizedFilename = NormalizeFilename(Directory, false);
#if !UNIX_PLATFORM_FILE_SPEEDUP_FILE_OPERATIONS
	if (!GCaseInsensMapper.MapCaseInsensitiveFile(NormalizedFilename, CaseSensitiveFilename))
	{
		// could not find the file
		return false;
	}
#else
	CaseSensitiveFilename = NormalizedFilename;
#endif

	struct stat FileInfo;
	if (stat(TCHAR_TO_UTF8(*CaseSensitiveFilename), &FileInfo) != -1)
	{
		return S_ISDIR(FileInfo.st_mode);
	}
	return false;
}

bool FUnixPlatformFile::CreateDirectory(const TCHAR* Directory)
{
	FString NormalizedPath = NormalizeFilename(Directory, true);
	if (!CreateDirectoriesFromPath(*NormalizedPath))
	{
		return false;
	}

	return mkdir(TCHAR_TO_UTF8(*NormalizedPath), 0775) == 0 || (errno == EEXIST);
}

bool FUnixPlatformFile::DeleteDirectory(const TCHAR* Directory)
{
	FString CaseSensitiveFilename;
	FString IntendedFilename(NormalizeFilename(Directory,true));
#if !UNIX_PLATFORM_FILE_SPEEDUP_FILE_OPERATIONS
	if (!GCaseInsensMapper.MapCaseInsensitiveFile(IntendedFilename, CaseSensitiveFilename))
	{
		// could not find the directory
		return false;
	}
#else
	CaseSensitiveFilename = IntendedFilename;
#endif

	{
		FileMapCache& Cache = GetFileMapCache();
		UE::TScopeLock Lock(Cache);
		GetFileMapCache().Invalidate(IntendedFilename);
	}

	// removing mapped directory is too dangerous
	if (IntendedFilename != CaseSensitiveFilename)
	{
		UE_LOGF(LogUnixPlatformFile, Warning, "Could not find directory '%ls', deleting '%ls' instead (for consistency with the rest of file ops)", *IntendedFilename, *CaseSensitiveFilename);
	}
	return rmdir(TCHAR_TO_UTF8(*CaseSensitiveFilename)) == 0;
}

FFileStatData FUnixPlatformFile::GetStatData(const TCHAR* FilenameOrDirectory)
{
	FString CaseSensitiveFilename;
	if (!GCaseInsensMapper.MapCaseInsensitiveFile(NormalizeFilename(FilenameOrDirectory, false), CaseSensitiveFilename))
	{
		// could not find the file
		return FFileStatData();
	}

	struct stat FileInfo;
	if(stat(TCHAR_TO_UTF8(*CaseSensitiveFilename), &FileInfo) == -1)
	{
		return FFileStatData();
	}

	return UnixStatToUEFileData(FileInfo);
}

bool FUnixPlatformFile::IterateDirectory(const TCHAR* Directory, FDirectoryVisitor& Visitor)
{
	const FString DirectoryStr = Directory;
	const FString NormalizedDirectoryStr = NormalizeFilename(Directory, false);

	return IterateDirectoryCommon(Directory, [&](struct dirent* InEntry) -> bool
	{
		const FString UnicodeEntryName = UTF8_TO_TCHAR(InEntry->d_name);
				
		bool bIsDirectory = false;
		if (InEntry->d_type != DT_UNKNOWN && InEntry->d_type != DT_LNK)
		{
			bIsDirectory = InEntry->d_type == DT_DIR;
		}
		else
		{
			// either filesystem does not support d_type (e.g. a network one or non-native) or we're dealing with a symbolic link, fallback to stat
			struct stat FileInfo;
			const FString AbsoluteUnicodeName = NormalizedDirectoryStr / UnicodeEntryName;
			if (stat(TCHAR_TO_UTF8(*AbsoluteUnicodeName), &FileInfo) != -1)
			{
				bIsDirectory = ((FileInfo.st_mode & S_IFMT) == S_IFDIR);
			}
			else
			{
				int ErrNo = errno;
				UE_LOGF(LogUnixPlatformFile, Warning, "Cannot determine whether '%ls' is a directory - d_type not supported and stat() failed with errno=%d (%ls)", *AbsoluteUnicodeName, ErrNo, UTF8_TO_TCHAR(strerror(ErrNo)));
			}
		}

		return Visitor.CallShouldVisitAndVisit(*(DirectoryStr / UnicodeEntryName), bIsDirectory);
	});
}

bool FUnixPlatformFile::IterateDirectoryStat(const TCHAR* Directory, FDirectoryStatVisitor& Visitor)
{
	const FString DirectoryStr = Directory;
	const FString NormalizedDirectoryStr = NormalizeFilename(Directory, false);

	return IterateDirectoryCommon(Directory, [&](struct dirent* InEntry) -> bool
	{
		const FString UnicodeEntryName = UTF8_TO_TCHAR(InEntry->d_name);
				
		struct stat FileInfo;
		const FString AbsoluteUnicodeName = NormalizedDirectoryStr / UnicodeEntryName;	
		if (stat(TCHAR_TO_UTF8(*AbsoluteUnicodeName), &FileInfo) != -1)
		{
			return Visitor.CallShouldVisitAndVisit(*(DirectoryStr / UnicodeEntryName), UnixStatToUEFileData(FileInfo));
		}

		return true;
	});
}

bool FUnixPlatformFile::IterateDirectoryCommon(const TCHAR* Directory, const TFunctionRef<bool(struct dirent*)>& Visitor)
{
	bool Result = false;

	FString NormalizedDirectory = NormalizeFilename(Directory, false);
	DIR* Handle = opendir(TCHAR_TO_UTF8(*NormalizedDirectory));
	if (Handle)
	{
		Result = true;
		struct dirent* Entry;
		while (Result && (Entry = readdir(Handle)) != NULL)
		{
			if (FCString::Strcmp(UTF8_TO_TCHAR(Entry->d_name), TEXT(".")) && FCString::Strcmp(UTF8_TO_TCHAR(Entry->d_name), TEXT("..")))
			{
				Result = Visitor(Entry);
			}
		}
		closedir(Handle);
	}
	return Result;
}

bool FUnixPlatformFile::CopyFile(const TCHAR* To, const TCHAR* From, EPlatformFileRead ReadFlags, EPlatformFileWrite WriteFlags)
{
	bool Result = IPlatformFile::CopyFile(To, From, ReadFlags, WriteFlags);
	if (Result)
	{
		struct stat FileInfo;
		if (stat(TCHAR_TO_UTF8(*NormalizeFilename(From, false)), &FileInfo) == 0)
		{
			FileInfo.st_mode |= S_IWUSR;
			chmod(TCHAR_TO_UTF8(*NormalizeFilename(To, true)), FileInfo.st_mode);
		}
	}
	return Result;
}

bool FUnixPlatformFile::CreateDirectoriesFromPath(const TCHAR* Path)
{
	// if the file already exists, then directories exist.
	struct stat FileInfo;
	if (stat(TCHAR_TO_UTF8(*NormalizeFilename(Path, true)), &FileInfo) != -1)
	{
		return true;
	}

	// convert path to native char string.
	int32 Len = strlen(TCHAR_TO_UTF8(*NormalizeFilename(Path, true)));
	char *DirPath = reinterpret_cast<char *>(FMemory::Malloc((Len+2) * sizeof(char)));
	char *SubPath = reinterpret_cast<char *>(FMemory::Malloc((Len+2) * sizeof(char)));
	strcpy(DirPath, TCHAR_TO_UTF8(*NormalizeFilename(Path, true)));

	for (int32 i=0; i<Len; ++i)
	{
		SubPath[i] = DirPath[i];

		if (SubPath[i] == '/')
		{
			SubPath[i+1] = 0;

			if (mkdir(SubPath, 0775) == -1)
			{
				int ErrNo = errno;

				// Folder already exists, continue and make sure the rest of the path is created
				if (ErrNo == EEXIST)
				{
					continue;
				}

				UE_LOGF_UNIX_FILE(Warning, "create dir('%s') failed: errno=%d (%s)", SubPath, ErrNo, strerror(ErrNo));

				FMemory::Free(DirPath);
				FMemory::Free(SubPath);

				return false;
			}
		}
	}

	FMemory::Free(DirPath);
	FMemory::Free(SubPath);
	return true;
}

FRegisteredFileHandle* FUnixFileRegistry::PlatformInitialOpenFile(const TCHAR* Filename)
{
	FString MappedToName;
	int32 Handle = GCaseInsensMapper.OpenCaseInsensitiveRead(Filename, MappedToName);
	if (Handle != -1)
	{
		return new FFileHandleUnix(Handle, *MappedToName, false);
	}
	return nullptr;
}

bool FUnixFileRegistry::PlatformReopenFile(FRegisteredFileHandle* Handle)
{
	FFileHandleUnix* UnixHandle = static_cast<FFileHandleUnix*>(Handle);

	bool bSuccess = true;
	UnixHandle->FileHandle = open(TCHAR_TO_UTF8(*(UnixHandle->Filename)), O_RDONLY | O_CLOEXEC);
	if(UnixHandle->FileHandle != -1)
	{
		if (lseek(UnixHandle->FileHandle, UnixHandle->FileOffset, SEEK_SET) == -1)
		{
			UE_LOGF(LogUnixPlatformFile, Warning, "Could not seek to the previous position on handle for file '%ls'", *(UnixHandle->Filename));
			bSuccess = false;
		}
	}
	else
	{
		UE_LOGF(LogUnixPlatformFile, Warning, "Could not reopen handle for file '%ls'", *(UnixHandle->Filename));
		bSuccess = false;
	}

	return bSuccess;
}

void FUnixFileRegistry::PlatformCloseFile(FRegisteredFileHandle* Handle)
{
	FFileHandleUnix* UnixHandle = static_cast<FFileHandleUnix*>(Handle);
	close(UnixHandle->FileHandle);
}

