// Copyright Epic Games, Inc. All Rights Reserved.

#include "Unix/UnixPlatformGuid.h"

#include "HAL/Platform.h"
#include "Misc/AssertionMacros.h"

#include <sys/random.h>
#include <time.h>

namespace
{
	// Set the 4-bit Version and 2-bit Variant fields of a FGuid.
	static inline void SetGuidType(FGuid& Target, uint8 Version, uint8 Variant)
	{
		Target.B = (Target.B & 0xffff0fff) | ((static_cast<uint32>(Version) & 0b1111) << 12);
		Target.C = (Target.C & 0x3fffffff) | ((static_cast<uint32>(Variant) & 0b11) << 30);
	}

} // namespace (anonymous)

FGuid FUnixPlatformGuid::CreateGuid()
{
	return CreateGuidFromTimeAndRandom();
}

FGuid FUnixPlatformGuid::CreateGuidFromTimeAndRandom()
{
	/**
	 * UUIDv7 format:
	 * - 48 bits Unix timestamp in milliseconds
	 * -  4 bits version (7)
	 * - 12 bits random data
	 * -  2 bits variant (0b10)
	 * - 62 bits random data
	 */
	struct timespec ts = {};
	if (!ensure(clock_gettime(CLOCK_REALTIME, &ts) == 0))
	{
		return FGuid();
	}

	const uint64 UnixTimestampMilliseconds = static_cast<uint64>(ts.tv_sec) * 1000 + ts.tv_nsec / 1'000'000;

	FGuid Guid = {};
	// Load random data from 2nd half of B through D (80 bits, although 6 bits will be overwritten with the version & variant fields).
	constexpr size_t NumRandomBytes = 10;
	constexpr unsigned int GetRandomFlags = 0;
	if (!ensure(getrandom(reinterpret_cast<char*>(&Guid.A) + 6, NumRandomBytes, GetRandomFlags) == NumRandomBytes))
	{
		return FGuid();
	}

#if PLATFORM_LITTLE_ENDIAN
	// Shift random half of Guid.B into the lower end.
	Guid.B >>= 16;
#endif

	Guid.A = static_cast<uint32>(UnixTimestampMilliseconds >> 16);
	Guid.B |= static_cast<uint32>(UnixTimestampMilliseconds) << 16;
	SetGuidType(Guid, 7, 0b10);
	return Guid;
}

FGuid FUnixPlatformGuid::CreateGuidFromRandomOnly()
{
	FGuid Guid = {};
	constexpr unsigned int GetRandomFlags = 0;
	if (!ensure(getrandom(&Guid, sizeof(Guid), GetRandomFlags) == sizeof(Guid)))
	{
		return FGuid();
	}

	SetGuidType(Guid, 4, 0b10);
	return Guid;
}
