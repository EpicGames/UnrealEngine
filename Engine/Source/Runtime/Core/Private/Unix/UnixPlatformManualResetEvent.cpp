// Copyright Epic Games, Inc. All Rights Reserved.

#include "Unix/UnixPlatformManualResetEvent.h"

#include "Async/Fundamental/Oversubscription.h"
#include "HAL/PlatformTime.h"

#include <limits>
#include <linux/futex.h>
#include <sys/syscall.h>
#include <time.h>
#include <unistd.h>

namespace UE::HAL::Private
{

inline static const struct timespec* SecondsToTimeSpec(double Seconds, struct timespec& OutTime)
{
	if (Seconds != std::numeric_limits<double>::infinity())
	{
		const double TotalSec = Seconds >= 0.0 ? Seconds : 0.0;
		OutTime.tv_sec = time_t(TotalSec);
		OutTime.tv_nsec = long((TotalSec - double(OutTime.tv_sec)) * 1e9);
		return &OutTime;
	}
	return nullptr;
}

FORCENOINLINE void FUnixManualResetEvent::WaitSlow()
{
	LowLevelTasks::FOversubscriptionScope _;

	for (;;)
	{
		if (State.load(std::memory_order_acquire))
		{
			return;
		}
		syscall(SYS_futex, &State, FUTEX_WAIT_PRIVATE, 0, nullptr, nullptr, 0);
	}
}

FORCENOINLINE bool FUnixManualResetEvent::WaitForSlow(FMonotonicTimeSpan WaitTime)
{
	if (State.load(std::memory_order_acquire))
	{
		return true;
	}

	LowLevelTasks::FOversubscriptionScope _(WaitTime > FMonotonicTimeSpan::Zero());
	struct timespec WaitTimeSpec;
	const struct timespec* WaitTimeSpecPtr = SecondsToTimeSpec(WaitTime.ToSeconds(), WaitTimeSpec);

	// FUTEX_WAIT takes a relative wait time.
	const int ClockOption = FPlatformTime::GetClockSource() == CLOCK_REALTIME ? FUTEX_CLOCK_REALTIME : 0;
	if (syscall(SYS_futex, &State, FUTEX_WAIT_PRIVATE | ClockOption, 0, WaitTimeSpecPtr, nullptr, 0) == -1 && errno == ETIMEDOUT)
	{
		return false;
	}

	if (LIKELY(State.load(std::memory_order_acquire)))
	{
		return true;
	}

	// Handle a spurious wake by waiting until the wait time has elapsed one more time because WaitUntilSlow
	// handles spurious wakes in a loop and avoids exceeding the originally requested wake time by more than
	// the typical variation due to scheduling imprecision.
	return WaitUntilSlow(FMonotonicTimePoint::Now() + WaitTime);
}

FORCENOINLINE bool FUnixManualResetEvent::WaitUntilSlow(FMonotonicTimePoint WaitTime)
{
	LowLevelTasks::FOversubscriptionScope _;
	struct timespec WaitTimeSpec;
	const struct timespec* WaitTimeSpecPtr = SecondsToTimeSpec(WaitTime.ToSeconds(), WaitTimeSpec);

	for (;;)
	{
		if (State.load(std::memory_order_acquire))
		{
			return true;
		}

		// FUTEX_WAIT_BITSET takes an absolute wait time.
		const int ClockOption = FPlatformTime::GetClockSource() == CLOCK_REALTIME ? FUTEX_CLOCK_REALTIME : 0;
		if (syscall(SYS_futex, &State, FUTEX_WAIT_BITSET_PRIVATE | ClockOption, 0, WaitTimeSpecPtr, nullptr, FUTEX_BITSET_MATCH_ANY) == -1 && errno == ETIMEDOUT)
		{
			return false;
		}
	}
}

void FUnixManualResetEvent::Notify()
{
	if (State.exchange(1, std::memory_order_release) == 0)
	{
		syscall(SYS_futex, &State, FUTEX_WAKE_PRIVATE, INT_MAX, nullptr, nullptr, 0);
	}
}

} // UE::HAL::Private
