// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/MonotonicTime.h"

#include <atomic>

namespace UE::HAL::Private
{

/** @see FGenericPlatformManualResetEvent */
class FUnixManualResetEvent
{
public:
	FUnixManualResetEvent() = default;
	FUnixManualResetEvent(const FUnixManualResetEvent&) = delete;
	FUnixManualResetEvent& operator=(const FUnixManualResetEvent&) = delete;

	inline void Reset()
	{
		State.store(0, std::memory_order_relaxed);
	}

	inline bool Poll()
	{
		return !!State.load(std::memory_order_acquire);
	}

	inline void Wait()
	{
		if (!State.load(std::memory_order_acquire))
		{
			WaitSlow();
		}
	}

	inline bool WaitFor(FMonotonicTimeSpan WaitTime)
	{
		return State.load(std::memory_order_acquire) || WaitForSlow(WaitTime);
	}

	inline bool WaitUntil(FMonotonicTimePoint WaitTime)
	{
		return State.load(std::memory_order_acquire) || WaitUntilSlow(WaitTime);
	}

	void Notify();

private:
	void WaitSlow();
	bool WaitForSlow(FMonotonicTimeSpan WaitTime);
	bool WaitUntilSlow(FMonotonicTimePoint WaitTime);

	// 0 = Reset, 1 = Notified
	std::atomic<uint32> State = 0;
};

} // UE::HAL::Private
