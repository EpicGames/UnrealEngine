// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	UnixPlatformMemory.cpp: Unix platform memory functions
=============================================================================*/

#include "Unix/UnixPlatformMemory.h"
#include "Unix/UnixForkPageProtector.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CoreDelegates.h"
#include "Misc/FeedbackContext.h"
#include "Math/NumericLimits.h"
#include "Math/UnrealMathUtility.h"
#include "Templates/UnrealTemplate.h"
#include "Containers/UnrealString.h"
#include "Logging/LogMacros.h"
#include "HAL/MallocAnsi.h"
#include "HAL/MallocMimalloc.h"
#include "HAL/MallocJemalloc.h"
#include "HAL/MallocBinned.h"
#include "HAL/MallocBinned2.h"
#include "HAL/MallocBinned3.h"
#include "HAL/MallocBinned4.h"
#include "HAL/MallocReplayProxy.h"
#include "HAL/MallocStomp.h"
#include "HAL/PlatformMallocCrash.h"
#include "ProfilingDebugging/CsvProfiler.h"

#include <sys/sysinfo.h>
#include <sys/file.h>
#include <sys/mman.h>

#include "GenericPlatform/OSAllocationPool.h"
#include "Misc/ScopeLock.h"

#ifdef UE4_DO_ROOT_PRIVILEGE_CHECK
	// UE_DEPRECATED(5.8)
	#warning "UE4_DO_ROOT_PRIVILEGE_CHECK has been renamed UE_DO_ROOT_PRIVILEGE_CHECK"
	#define UE_DO_ROOT_PRIVILEGE_CHECK UE4_DO_ROOT_PRIVILEGE_CHECK
#endif
// do not do a root privilege check on non-x86-64 platforms (assume an embedded device)
#ifndef UE_DO_ROOT_PRIVILEGE_CHECK
  #if defined(_M_X64) || defined(__x86_64__) || defined (__amd64__)
#define UE_DO_ROOT_PRIVILEGE_CHECK	 1
  #else
    #define UE_DO_ROOT_PRIVILEGE_CHECK	 0
  #endif // defined(_M_X64) || defined(__x86_64__) || defined (__amd64__) 
#endif // ifndef UE_DO_ROOT_PRIVILEGE_CHECK

// Set to 1 if we should try to use /proc/self/smaps_rollup in FUnixPlatformMemory::GetExtendedStats().
// There is a potential tradeoff in that smaps_rollup appears to be quite a bit faster in
//   straight testing, but it also looks like it blocks mmap() calls on other threads until it finishes.
#ifndef USE_PROC_SELF_SMAPS_ROLLUP
#define USE_PROC_SELF_SMAPS_ROLLUP 0
#endif

// Used in UnixPlatformStackwalk to skip the crash handling callstack frames.
bool CORE_API GFullCrashCallstack = false;

// Used to enable kernel shared memory from mmap'd memory
bool CORE_API GUseKSM = false;
bool CORE_API GKSMMergeAllPages = false;

// Used to enable or disable timing of ensures. Enabled by default
bool CORE_API GTimeEnsures = true;

// Allows settings a specific signal to maintain its default handler rather then ignoring the signal
int32 CORE_API GSignalToDefault = 0;

// Allows setting crash handler stack size
uint64 CORE_API GCrashHandlerStackSize = 0;

// Due to dotnet not allowing any files marked as LOCK_EX to be opened for read only or copied, this allows us to
// to disable the locking mechanics. https://github.com/dotnet/runtime/issues/34126
// Default to true, can be disabled with -noexclusivelockonwrite
bool GAllowExclusiveLockOnWrite = true;

#if UE_SERVER
// Scale factor for how much we would like to increase or decrease the memory pool size
float CORE_API GPoolTableScale = 1.0f;
#endif

// Used to set the maximum number of file mappings.
#if UE_EDITOR
int32 CORE_API GMaxNumberFileMappingCache = 10000;
#else
int32 CORE_API GMaxNumberFileMappingCache = 100;
#endif

namespace
{
	// The max allowed to be set for the caching
	const int32 MaximumAllowedMaxNumFileMappingCache = 1000000;
	bool GEnableProtectForkedPages = false;
	std::atomic<uint64> GSharedMemoryCOWBytes = 0;
}

CSV_DECLARE_CATEGORY_EXTERN(FMemory);

/** Controls growth of pools - see PooledVirtualMemoryAllocator.cpp */
extern float GVMAPoolScale;

/** Make Decommit no-op (this significantly speeds up freeing memory at the expense of larger resident footprint) */
bool GMemoryRangeDecommitIsNoOp = (UE_SERVER == 0);

void FUnixPlatformMemory::Init()
{
	// Only allow this method to be called once
	{
		static bool bInitDone = false;
		if (bInitDone)
			return;
		bInitDone = true;
	}

	FGenericPlatformMemory::Init();

	const FPlatformMemoryConstants& MemoryConstants = FPlatformMemory::GetConstants();
	UE_LOGF(LogInit, Log, " - Physical RAM available (not considering process quota): %d GB (%lu MB, %lu KB, %lu bytes)", 
		MemoryConstants.TotalPhysicalGB, 
		MemoryConstants.TotalPhysical / ( 1024ULL * 1024ULL ), 
		MemoryConstants.TotalPhysical / 1024ULL, 
		MemoryConstants.TotalPhysical);
	UE_LOGF(LogInit, Log, " - VirtualMemoryAllocator pools will grow at scale %g", GVMAPoolScale);
	UE_LOGF(LogInit, Log, " - MemoryRangeDecommit() will %ls", 
		GMemoryRangeDecommitIsNoOp ? TEXT("be a no-op (re-run with -vmapoolevict to change)") : TEXT("will evict the memory from RAM (re-run with -novmapoolevict to change)"));
	UE_LOGF(LogInit, Log, " - PageSize %zu", MemoryConstants.PageSize);
	UE_LOGF(LogInit, Log, " - BinnedPageSize %zu", MemoryConstants.BinnedPageSize);
}

bool FUnixPlatformMemory::HasForkPageProtectorEnabled()
{
	return COMPILE_FORK_PAGE_PROTECTOR && GEnableProtectForkedPages;
}

class FMalloc* FUnixPlatformMemory::BaseAllocator()
{
	static FMalloc* Allocator = nullptr;
	if (Allocator != nullptr)
	{
		return Allocator;
	}

#if UE_DO_ROOT_PRIVILEGE_CHECK && !IS_PROGRAM
	// This function gets executed very early, way before main() (because global constructors will allocate memory).
	// This makes it ideal, if unobvious, place for a root privilege check.
	if (geteuid() == 0)
	{
		fprintf(stderr, "Refusing to run with the root privileges.\n");
		FPlatformMisc::RequestExit(true, TEXT("FUnixPlatformMemory.BaseAllocator"));
		// unreachable
		return nullptr;
	}
#endif // UE_DO_ROOT_PRIVILEGE_CHECK && !IS_PROGRAM

#if UE_USE_MALLOC_REPLAY_PROXY
	bool bAddReplayProxy = false;
#endif // UE_USE_MALLOC_REPLAY_PROXY

	if (USE_MALLOC_BINNED2)
	{
		AllocatorToUse = EMemoryAllocatorToUse::Binned2;
	}
	else if (USE_MALLOC_BINNED3)
	{
		AllocatorToUse = EMemoryAllocatorToUse::Binned3;
	}
	else
	{
		AllocatorToUse = EMemoryAllocatorToUse::Binned;
	}

	// Mimalloc is now the default allocator for editor and programs because it has shown
	// both great performance and as much as half the memory usage of TBB after
	// heavy editor workloads. See CL 15887498 description for benchmarks.
#if (WITH_EDITORONLY_DATA || IS_PROGRAM) && MIMALLOC_ENABLED
	AllocatorToUse = EMemoryAllocatorToUse::Mimalloc;
#endif

	// Allow overriding on the command line.
	// We get here before main due to global ctors, so need to do some hackery to get command line args
	if (FILE* CmdLineFile = fopen("/proc/self/cmdline", "r"))
	{
		char * Arg = nullptr;
		size_t Size = 0;
		while(getdelim(&Arg, &Size, 0, CmdLineFile) != -1)
		{
#if PLATFORM_SUPPORTS_JEMALLOC
			if (FCStringAnsi::Stricmp(Arg, "-jemalloc") == 0)
			{
				AllocatorToUse = EMemoryAllocatorToUse::Jemalloc;
				break;
			}
#endif // PLATFORM_SUPPORTS_JEMALLOC
			if (FCStringAnsi::Stricmp(Arg, "-ansimalloc") == 0)
			{
				// see FPlatformMisc::GetProcessDiagnostics()
				AllocatorToUse = EMemoryAllocatorToUse::Ansi;
				break;
			}

			if (FCStringAnsi::Stricmp(Arg, "-binnedmalloc") == 0)
			{
				AllocatorToUse = EMemoryAllocatorToUse::Binned;
				break;
			}

#if MIMALLOC_ENABLED
			if (FCStringAnsi::Stricmp(Arg, "-mimalloc") == 0)
			{
				AllocatorToUse = EMemoryAllocatorToUse::Mimalloc;
				break;
			}
#endif

			if (FCStringAnsi::Stricmp(Arg, "-binnedmalloc2") == 0)
			{
				AllocatorToUse = EMemoryAllocatorToUse::Binned2;
				break;
			}

			if (FCStringAnsi::Stricmp(Arg, "-binnedmalloc3") == 0)
			{
				AllocatorToUse = EMemoryAllocatorToUse::Binned3;
				break;
			}

			if (FCStringAnsi::Stricmp(Arg, "-binnedmalloc4") == 0)
			{
				AllocatorToUse = EMemoryAllocatorToUse::Binned4;
				break;
			}

			if (FCStringAnsi::Stricmp(Arg, "-fullcrashcallstack") == 0)
			{
				GFullCrashCallstack = true;
			}

			if (FCStringAnsi::Stricmp(Arg, "-useksm") == 0)
			{
				GUseKSM = true;
			}

			if (FCStringAnsi::Stricmp(Arg, "-ksmmergeall") == 0)
			{
				GKSMMergeAllPages = true;
			}

			if (FCStringAnsi::Stricmp(Arg, "-noensuretiming") == 0)
			{
				GTimeEnsures = false;
			}

			if (FCStringAnsi::Stricmp(Arg, "-noexclusivelockonwrite") == 0)
			{
				GAllowExclusiveLockOnWrite = false;
			}

			const char SignalToDefaultCmd[] = "-sigdfl=";
			if (const char* Cmd = FCStringAnsi::Stristr(Arg, SignalToDefaultCmd))
			{
				int32 SignalToDefault = FCStringAnsi::Atoi(Cmd + sizeof(SignalToDefaultCmd) - 1);

				// Valid signals are only from 1 -> SIGRTMAX
				if (SignalToDefault > SIGRTMAX)
				{
					SignalToDefault = 0;
				}

				GSignalToDefault = FMath::Max(SignalToDefault, 0);
			}

			const char CrashHandlerStackSize[] = "-crashhandlerstacksize=";
			if (const char* Cmd = FCStringAnsi::Stristr(Arg, CrashHandlerStackSize))
			{
				GCrashHandlerStackSize = FCStringAnsi::Atoi64(Cmd + sizeof(CrashHandlerStackSize) - 1);
			}

			const char FileMapCacheCmd[] = "-filemapcachesize=";
			if (const char* Cmd = FCStringAnsi::Stristr(Arg, FileMapCacheCmd))
			{
				int32 Max = FCStringAnsi::Atoi(Cmd + sizeof(FileMapCacheCmd) - 1);
				GMaxNumberFileMappingCache = FMath::Clamp(Max, 0, MaximumAllowedMaxNumFileMappingCache);
			}

#if UE_USE_MALLOC_REPLAY_PROXY
			if (FCStringAnsi::Stricmp(Arg, "-mallocsavereplay") == 0)
			{
				bAddReplayProxy = true;
			}
#endif // UE_USE_MALLOC_REPLAY_PROXY
#if WITH_MALLOC_STOMP
			if (FCStringAnsi::Stricmp(Arg, "-stompmalloc") == 0)
			{
				// see FPlatformMisc::GetProcessDiagnostics()
				AllocatorToUse = EMemoryAllocatorToUse::Stomp;
				break;
			}
#endif // WITH_MALLOC_STOMP

			const char VMAPoolScaleSwitch[] = "-vmapoolscale=";
			if (const char* Cmd = FCStringAnsi::Stristr(Arg, VMAPoolScaleSwitch))
			{
				float PoolScale = FCStringAnsi::Atof(Cmd + sizeof(VMAPoolScaleSwitch) - 1);
				GVMAPoolScale = FMath::Max(PoolScale, 1.0f);
			}

			if (FCStringAnsi::Stricmp(Arg, "-vmapoolevict") == 0)
			{
				GMemoryRangeDecommitIsNoOp = false;
			}
			if (FCStringAnsi::Stricmp(Arg, "-novmapoolevict") == 0)
			{
				GMemoryRangeDecommitIsNoOp = true;
			}
			if (FCStringAnsi::Stricmp(Arg, "-protectforkedpages") == 0)
			{
				GEnableProtectForkedPages = true;
			}
		}
		free(Arg);
		fclose(CmdLineFile);
	}

	// This was moved to the fact that we aboved the command line statements above to *include* other things besides allocator only switches
	// Moving here allows the other globals to be set, while we override the ANSI allocator still no matter the command line options
	if (FORCE_ANSI_ALLOCATOR)
	{
		AllocatorToUse = EMemoryAllocatorToUse::Ansi;
	}

	switch (AllocatorToUse)
	{
	case EMemoryAllocatorToUse::Ansi:
		Allocator = new FMallocAnsi();
		break;

#if WITH_MALLOC_STOMP
	case EMemoryAllocatorToUse::Stomp:
		Allocator = new FMallocStomp();
		break;
#endif

#if PLATFORM_SUPPORTS_JEMALLOC
	case EMemoryAllocatorToUse::Jemalloc:
		Allocator = new FMallocJemalloc();
		break;
#endif // PLATFORM_SUPPORTS_JEMALLOC

#if MIMALLOC_ENABLED
	case EMemoryAllocatorToUse::Mimalloc:
		Allocator = new FMallocMimalloc();
		break;
#endif

	case EMemoryAllocatorToUse::Binned2:
		Allocator = new FMallocBinned2();
		break;

	case EMemoryAllocatorToUse::Binned3:
		Allocator = new FMallocBinned3();
		break;

	case EMemoryAllocatorToUse::Binned4:
		Allocator = new FMallocBinned4();
		break;

	default:	// intentional fall-through
	case EMemoryAllocatorToUse::Binned:
		Allocator = new FMallocBinned(FPlatformMemory::GetConstants().BinnedPageSize & MAX_uint32, 0x100000000);
		break;
	}

#if UE_BUILD_DEBUG
	printf("Using %s.\n", Allocator ? TCHAR_TO_UTF8(Allocator->GetDescriptiveName()) : "NULL allocator! We will probably crash right away");
#endif // UE_BUILD_DEBUG

#if UE_USE_MALLOC_REPLAY_PROXY
	if (bAddReplayProxy)
	{
		Allocator = new FMallocReplayProxy(Allocator);
	}
#endif // UE_USE_MALLOC_REPLAY_PROXY

	return Allocator;
}

bool FUnixPlatformMemory::PageProtect(void* const Ptr, const SIZE_T Size, const bool bCanRead, const bool bCanWrite)
{
	int32 ProtectMode;
	if (bCanRead && bCanWrite)
	{
		ProtectMode = PROT_READ | PROT_WRITE;
	}
	else if (bCanRead)
	{
		ProtectMode = PROT_READ;
	}
	else if (bCanWrite)
	{
		ProtectMode = PROT_WRITE;
	}
	else
	{
		ProtectMode = PROT_NONE;
	}
	return mprotect(Ptr, Size, ProtectMode) == 0;
}


static void MarkMappedMemoryMergable(void* Pointer, SIZE_T Size)
{
	const SIZE_T BinnedPageSize = FPlatformMemory::GetConstants().BinnedPageSize;

	// If we dont want to merge all pages only merge chunks larger then BinnedPageSize
	if (GUseKSM && (GKSMMergeAllPages || Size > BinnedPageSize))
	{
		int Ret = madvise(Pointer, Size, MADV_MERGEABLE);
		if (Ret != 0)
		{
			int ErrNo = errno;
			UE_LOGF(LogHAL, Fatal, "madvise(addr=%p, length=%zu, advice=MADV_MERGEABLE) failed with errno = %d (%ls)",
				Pointer, Size, ErrNo, StringCast< TCHAR >(strerror(ErrNo)).Get());
			// unreachable
		}
	}
}

#ifndef MALLOC_LEAKDETECTION
	#define MALLOC_LEAKDETECTION 0
#endif
// check bookkeeping info against the passed in parameters in Debug and Development (the latter only in games and servers. also, only if leak detection is disabled, otherwise things are very slow)
#define UE_PLATFORM_SANITY_CHECK_OS_ALLOCATIONS			(UE_BUILD_DEBUG || (UE_BUILD_DEVELOPMENT && (UE_GAME || UE_SERVER) && !MALLOC_LEAKDETECTION))

/** This structure is stored in the page after each OS allocation and checks that its properties are valid on Free. Should be less than the page size (4096 on all platforms we support) */
struct FOSAllocationDescriptor
{
	enum class MagicType : uint64
	{
		Marker = 0xd0c233ccf493dfb0
	};

	/** Magic that makes sure that we are not passed a pointer somewhere into the middle of the allocation (and/or the structure wasn't stomped). */
	MagicType	Magic;

	/** This should include the descriptor itself. */
	void*		PointerToUnmap;

	/** This should include the total size of allocation, so after unmapping it everything is gone, including the descriptor */
	SIZE_T		SizeToUnmap;

	/** Debug info that makes sure that the correct size is preserved. */
	SIZE_T		OriginalSizeAsPassed;
};

void* FUnixPlatformMemory::BinnedAllocFromOS(SIZE_T Size)
{
#if UE_CHECK_LARGE_ALLOCATIONS
	if (UE::Memory::Private::GEnableLargeAllocationChecks)
	{
		// catch possibly erroneous large allocations
		ensureMsgf(Size <= UE::Memory::Private::GLargeAllocationThreshold,
			TEXT("Single allocation exceeded large allocation threshold"));
	}
#endif

	static SIZE_T OSPageSize = FPlatformMemory::GetConstants().PageSize;
	// guard against someone not passing size in whole pages
	SIZE_T SizeInWholePages = (Size % OSPageSize) ? (Size + OSPageSize - (Size % OSPageSize)) : Size;
	void* Pointer = nullptr;

	// Binned expects OS allocations to be BinnedPageSize-aligned, and that page is at least 64KB. mmap() alone cannot do this, so carve out the needed chunks.
	const SIZE_T ExpectedAlignment = FPlatformMemory::GetConstants().BinnedPageSize;
	// Descriptor is only used if we're sanity checking. However, #ifdef'ing its use would make the code more fragile. Size needs to be at least one page.
#if (UE_PLATFORM_SANITY_CHECK_OS_ALLOCATIONS != 0)
	const SIZE_T DescriptorSize = OSPageSize;
#else
	const SIZE_T DescriptorSize = 0;
#endif

	SIZE_T ActualSizeMapped = SizeInWholePages + ExpectedAlignment;

	// the remainder of the map will be used for the descriptor, if any.
	// we always allocate at least one page more than needed
	void* PointerWeGotFromMMap = mmap(nullptr, ActualSizeMapped, PROT_READ | PROT_WRITE, MAP_PRIVATE | MAP_ANON, -1, 0);
	// store these values, to unmap later

	Pointer = PointerWeGotFromMMap;
	if (Pointer == MAP_FAILED)
	{
		FPlatformMemory::OnOutOfMemory(ActualSizeMapped, ExpectedAlignment);
		// unreachable
	}

	SIZE_T Offset = (reinterpret_cast<SIZE_T>(Pointer) % ExpectedAlignment);

	// See if we need to unmap anything in the front. If the pointer happened to be aligned, we don't need to unmap anything.
	if (LIKELY(Offset != 0))
	{
		// figure out how much to unmap before the alignment.
		SIZE_T SizeToNextAlignedPointer = ExpectedAlignment - Offset;
		void* AlignedPointer = reinterpret_cast<void*>(reinterpret_cast<SIZE_T>(Pointer) + SizeToNextAlignedPointer);

		// unmap the part before
		if (munmap(Pointer, SizeToNextAlignedPointer) != 0)
		{
			FPlatformMemory::OnOutOfMemory(SizeToNextAlignedPointer, ExpectedAlignment);
			// unreachable
		}

		// account for reduced mmaped size
		ActualSizeMapped -= SizeToNextAlignedPointer;

		// now, make it appear as if we initially got the allocation right
		Pointer = AlignedPointer;
	}

	MarkMappedMemoryMergable(Pointer, ActualSizeMapped);

	// at this point, Pointer is aligned at the expected alignment - either we lucked out on the initial allocation
	// or we already got rid of the extra memory that was allocated in the front.
	checkf((reinterpret_cast<SIZE_T>(Pointer) % ExpectedAlignment) == 0, TEXT("BinnedAllocFromOS(): Internal error: did not align the pointer as expected."));

	// Now unmap the tail only, if any, but leave just enough space for the descriptor
	void* TailPtr = reinterpret_cast<void*>(reinterpret_cast<SIZE_T>(Pointer) + SizeInWholePages + DescriptorSize);
	SIZE_T TailSize = ActualSizeMapped - SizeInWholePages - DescriptorSize;

	if (LIKELY(TailSize > 0))
	{
		if (munmap(TailPtr, TailSize) != 0)
		{
			FPlatformMemory::OnOutOfMemory(TailSize, ExpectedAlignment);
			// unreachable
		}
	}

#if (UE_PLATFORM_SANITY_CHECK_OS_ALLOCATIONS != 0)
	// we're done with this allocation, fill in the descriptor with the info
	if (LIKELY(DescriptorSize > 0))
	{
		FOSAllocationDescriptor* AllocDescriptor = reinterpret_cast<FOSAllocationDescriptor*>(reinterpret_cast<SIZE_T>(Pointer) + Size);
		AllocDescriptor->Magic = FOSAllocationDescriptor::MagicType::Marker;
		AllocDescriptor->PointerToUnmap = Pointer;
		AllocDescriptor->SizeToUnmap = SizeInWholePages + DescriptorSize;
		AllocDescriptor->OriginalSizeAsPassed = Size;
	}
#endif

	LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelAlloc(ELLMTracker::Platform, Pointer, Size));
	UE::FForkPageProtector::Get().AddMemoryRegion(Pointer, Size);

	return Pointer;
}

void FUnixPlatformMemory::BinnedFreeToOS(void* Ptr, SIZE_T Size)
{
	LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelFree(ELLMTracker::Platform, Ptr));
	// guard against someone not passing size in whole pages
	static SIZE_T OSPageSize = FPlatformMemory::GetConstants().PageSize;
	SIZE_T SizeInWholePages = (Size % OSPageSize) ? (Size + OSPageSize - (Size % OSPageSize)) : Size;

	#if (UE_PLATFORM_SANITY_CHECK_OS_ALLOCATIONS)
	{
		const SIZE_T DescriptorSize = OSPageSize;

		FOSAllocationDescriptor* AllocDescriptor = reinterpret_cast<FOSAllocationDescriptor*>(reinterpret_cast<SIZE_T>(Ptr) + Size);
		if (UNLIKELY(AllocDescriptor->Magic != FOSAllocationDescriptor::MagicType::Marker))
		{
			UE_LOGF(LogHAL, Fatal, "BinnedFreeToOS() has been passed an address %p (size %llu) not allocated through it.", Ptr, (uint64)Size);
			// unreachable
			return;
		}

		void* PointerToUnmap = AllocDescriptor->PointerToUnmap;
		SIZE_T SizeToUnmap = AllocDescriptor->SizeToUnmap;

		UE::FForkPageProtector::Get().FreeMemoryRegion(PointerToUnmap);

		if (UNLIKELY(PointerToUnmap != Ptr || SizeToUnmap != SizeInWholePages + DescriptorSize))
		{
			UE_LOGF(LogHAL, Fatal, "BinnedFreeToOS(): info mismatch: descriptor ptr: %p, size %llu, but our pointer is %p and size %llu.", PointerToUnmap, SizeToUnmap, AllocDescriptor, (uint64)(SizeInWholePages + DescriptorSize));
			// unreachable
			return;
		}

		if (UNLIKELY(AllocDescriptor->OriginalSizeAsPassed != Size))
		{
			UE_LOGF(LogHAL, Fatal, "BinnedFreeToOS(): info mismatch: descriptor original size %llu, our size is %llu for pointer %p", AllocDescriptor->OriginalSizeAsPassed, Size, Ptr);
			// unreachable
			return;
		}

		AllocDescriptor = nullptr;	// just so no one touches it

		if (UNLIKELY(munmap(PointerToUnmap, SizeToUnmap) != 0))
		{
			FPlatformMemory::OnOutOfMemory(SizeToUnmap, 0);
			// unreachable
		}
	}
	#else
	{
		UE::FForkPageProtector::Get().FreeMemoryRegion(Ptr);

		if (UNLIKELY(munmap(Ptr, SizeInWholePages) != 0))
		{
			FPlatformMemory::OnOutOfMemory(SizeInWholePages, 0);
			// unreachable
		}
	}
	#endif
}

size_t FUnixPlatformMemory::FPlatformVirtualMemoryBlock::GetVirtualSizeAlignment()
{
	static SIZE_T OSPageSize = FPlatformMemory::GetConstants().PageSize;
	return OSPageSize;
}

size_t FUnixPlatformMemory::FPlatformVirtualMemoryBlock::GetCommitAlignment()
{
	static SIZE_T OSPageSize = FPlatformMemory::GetConstants().PageSize;
	return OSPageSize;
}

FUnixPlatformMemory::FPlatformVirtualMemoryBlock FUnixPlatformMemory::FPlatformVirtualMemoryBlock::AllocateVirtual(size_t InSize, size_t InAlignment)
{
	FPlatformVirtualMemoryBlock Result;
	InSize = Align(InSize, GetVirtualSizeAlignment());
	Result.VMSizeDivVirtualSizeAlignment = InSize / GetVirtualSizeAlignment();

	size_t Alignment = FMath::Max(InAlignment, GetVirtualSizeAlignment());
	check(Alignment <= GetVirtualSizeAlignment());

	Result.Ptr = mmap(nullptr, Result.GetActualSize(), PROT_READ | PROT_WRITE, MAP_PRIVATE | MAP_ANON | MAP_NORESERVE, -1, 0);
	if (LIKELY(Result.Ptr != MAP_FAILED))
	{
		MarkMappedMemoryMergable(Result.Ptr, Result.GetActualSize());

		UE::FForkPageProtector::Get().AddMemoryRegion(Result.Ptr, Result.GetActualSize());
	}
	else
	{
		FPlatformMemory::OnOutOfMemory(Result.GetActualSize(), InAlignment);
	}
	check(Result.Ptr && IsAligned(Result.Ptr, Alignment));
	return Result;
}



void FUnixPlatformMemory::FPlatformVirtualMemoryBlock::FreeVirtual()
{
	if (Ptr)
	{
		check(GetActualSize() > 0);
		if (munmap(Ptr, GetActualSize()) != 0)
		{
			// we can ran out of VMAs here
			FPlatformMemory::OnOutOfMemory(GetActualSize(), 0);
			// unreachable
		}

		UE::FForkPageProtector::Get().FreeMemoryRegion(Ptr);

		Ptr = nullptr;
		VMSizeDivVirtualSizeAlignment = 0;
	}
}

void FUnixPlatformMemory::FPlatformVirtualMemoryBlock::Commit(size_t InOffset, size_t InSize)
{
	check(IsAligned(InOffset, GetCommitAlignment()) && IsAligned(InSize, GetCommitAlignment()));
	check(InOffset >= 0 && InSize >= 0 && InOffset + InSize <= GetActualSize() && Ptr);
}

void FUnixPlatformMemory::FPlatformVirtualMemoryBlock::Decommit(size_t InOffset, size_t InSize)
{
	check(IsAligned(InOffset, GetCommitAlignment()) && IsAligned(InSize, GetCommitAlignment()));
	check(InOffset >= 0 && InSize >= 0 && InOffset + InSize <= GetActualSize() && Ptr);
	if (!LIKELY(GMemoryRangeDecommitIsNoOp))
	{
		if (madvise(((uint8*)Ptr) + InOffset, InSize, MADV_DONTNEED) != 0)
		{
			// we can ran out of VMAs here too!
			FPlatformMemory::OnOutOfMemory(InSize, 0);
		}
	}
}

struct FProcField
{
	const ANSICHAR* Name = nullptr;
	uint64* Addr = nullptr;
	uint32 NameLen = 0;

	FProcField(const ANSICHAR *NameIn, uint64* AddrIn)
		: Name(NameIn), Addr(AddrIn)
	{
		NameLen = FCStringAnsi::Strlen(Name);
	}
};

static uint64 ParseProcFieldsChunk(ANSICHAR *Buffer, uint64 BufferSize, FProcField *ProcFields, uint32 NumFields, uint32 &NumFieldsFound)
{
	uint64 ParsePos = 0;

	for (uint64 Idx = 0; Idx < BufferSize; Idx++)
	{
		if (Buffer[Idx] == '\n')
		{
			const ANSICHAR *Line = Buffer + ParsePos;

			Buffer[Idx] = 0;

			for (uint32 IdxField = 0; IdxField < NumFields; IdxField++)
			{
				uint32 NameLen = ProcFields[IdxField].NameLen;

				if (!FCStringAnsi::Strncmp(Line, ProcFields[IdxField].Name, NameLen))
				{
					*ProcFields[IdxField].Addr = atoll(Line + NameLen) * 1024ULL;
					NumFieldsFound++;
					break;
				}
			}

			ParsePos = Idx + 1;
		}
	}

	// If we didn't find any linefeeds, skip the entire chunk
	return ParsePos ? ParsePos : BufferSize;
}

static uint32 ReadProcFields(const char *FileName, FProcField *ProcFields, uint32 NumFields)
{
	uint32 NumFieldsFound = 0;

	int Fd = open(FileName, O_RDONLY);
	if (Fd >= 0)
	{
		const uint32 ChunkSize = 512;
		ANSICHAR Buffer[ChunkSize];
		uint64 BytesAvailableInChunk = 0;

		for (;;)
		{
			ssize_t BytesRead;
			uint64 BytesToRead = ChunkSize - BytesAvailableInChunk;

			do
			{
				BytesRead = read(Fd, Buffer + BytesAvailableInChunk, BytesToRead);
			} while (BytesRead < 0 && errno == EINTR);

			if (BytesRead <= 0)
			{
				break;
			}

			BytesAvailableInChunk += BytesRead;

			uint64 BytesParsed = ParseProcFieldsChunk(Buffer, BytesAvailableInChunk, ProcFields, NumFields, NumFieldsFound);
			checkf(BytesParsed <= BytesAvailableInChunk, TEXT("BytesParsed more than BytesAvailableInChunk %llu %llu"), BytesParsed, BytesAvailableInChunk);

			if (BytesRead < BytesToRead || NumFieldsFound == NumFields)
			{
				break;
			}

			BytesAvailableInChunk -= BytesParsed;
			memmove(Buffer, Buffer + BytesParsed, BytesAvailableInChunk);
		}

		close(Fd);

		if (NumFieldsFound != NumFields)
		{
			static bool bLogOnceMissing = false;
			if (!bLogOnceMissing)
			{
				// Note: We can't use UE_LOG or TCHAR_TO_UTF8 since these routines may not be initialized yet and
				// allocate memory. This function could get called via FMemory::GCreateMalloc or signal handlers
				// and we will potentiall crash in these cases calling UE_LOG or TCHAR_TO_UTF8.
				fprintf(stderr, "Warning: ReadProcFields: %u of %u fields found in %s.\n",
						NumFieldsFound, NumFields, FileName);
				fflush(stderr);
				bLogOnceMissing = true;
			}
		}
	}
	else
	{
		static bool bLogOnceFileNotFound = false;
		if (!bLogOnceFileNotFound)
		{
			fprintf(stderr, "Warning: ReadProcFields failed opening %s (Err %d).\n", FileName, errno);
			fflush(stderr);
			bLogOnceFileNotFound = true;
		}
	}

	return NumFieldsFound;
}

static float ReadOvercommitRatio()
{
	float OutVal = 0.0f;
	int Fd = open("/proc/sys/vm/overcommit_ratio", O_RDONLY);

	if (Fd < 0)
	{
		static bool bLogOnceFileNotFound = false;
		if (!bLogOnceFileNotFound)
		{
			fprintf(stderr, "Warning: ReadOvercommitRatio failed to open /proc/sys/vm/overcommit_ratio (Err %d).\n", errno);
			fflush(stderr);
			bLogOnceFileNotFound = true;
		}
	}
	else
	{
		// The overcommit_ratio file always contains a number from 0 to 100 and nothing else
		char Buffer[512] = { 0 };
		ssize_t ReadBytes = read(Fd, Buffer, sizeof(Buffer) - 1);
		if (ReadBytes > 0)
		{
			OutVal = FCStringAnsi::Atof(Buffer) / 100.0f;
		}

		close(Fd);
	}

	return OutVal;
}

static int ReadOvercommitMode()
{
	int PolicyType = 2;	// Do not overcommit by default
	int Fd = open("/proc/sys/vm/overcommit_memory", O_RDONLY);

	if (Fd < 0)
	{
		static bool bLogOnceFileNotFound = false;
		if (!bLogOnceFileNotFound)
		{
			fprintf(stderr, "Warning: ReadOvercommitMode failed to open /proc/sys/vm/overcommit_memory (Err %d).\n", errno);
			fflush(stderr);
			bLogOnceFileNotFound = true;
		}
	}
	else
	{
		// The overcommit_memory policy is defined by a number from 0 to 2
		// https://www.kernel.org/doc/Documentation/vm/overcommit-accounting
		char Buffer[512] = { 0 };
		ssize_t ReadBytes = read(Fd, Buffer, sizeof(Buffer) - 1);
		if (ReadBytes > 0)
		{
			PolicyType = FCStringAnsi::Atoi(Buffer);
		}

		close(Fd);
	}

	return PolicyType;
}

// struct CORE_API FGenericPlatformMemoryStats : public FPlatformMemoryConstants
//   uint64 AvailablePhysical; /** The amount of physical memory currently available, in bytes. */			MemAvailable (or MemFree + Cached)
//   uint64 AvailableVirtual;  /** The amount of virtual memory currently available, in bytes. */			SwapFree
//   uint64 UsedPhysical;      /** The amount of physical memory used by the process, in bytes. */			VmRSS
//   uint64 PeakUsedPhysical;  /** The peak amount of physical memory used by the process, in bytes. */	VmHWM
//   uint64 UsedVirtual;       /** Total amount of virtual memory used by the process. */					VmSize
//   uint64 PeakUsedVirtual;   /** The peak amount of virtual memory used by the process. */				VmPeak
FPlatformMemoryStats FUnixPlatformMemory::GetStats()
{
	uint64 MemFree = 0;
	uint64 Cached = 0;
	uint64 MemTotal = 0;
	uint64 SwapTotal = 0;
	uint64 CommittedAS = 0;
	FPlatformMemoryStats MemoryStats;

	FProcField SMapsFields[] =
	{
		// An estimate of how much memory is available for starting new applications, without swapping.
		{ "MemAvailable:", &MemoryStats.AvailablePhysical },
		{ "MemFree:",      &MemFree },
		{ "Cached:",       &Cached },
		{ "MemTotal:",     &MemTotal },
		{ "SwapTotal:",    &SwapTotal },
		{ "Committed_AS:", &CommittedAS },
	};
	if (ReadProcFields("/proc/meminfo", SMapsFields, UE_ARRAY_COUNT(SMapsFields)))
	{
		// if we didn't have MemAvailable (kernels < 3.14 or CentOS 6.x), use free + cached as a (bad) approximation
		if (MemoryStats.AvailablePhysical == 0)
		{
			MemoryStats.AvailablePhysical = FMath::Min(MemFree + Cached, MemoryStats.TotalPhysical);
		}

		// OS alloted percentage of physical RAM (used here as a ratio) to overcommit
		// https://www.kernel.org/doc/Documentation/vm/overcommit-accounting
		float OvercommitRatio = ReadOvercommitRatio();

		// Total memory * commit percentage (which can be greater than 100%) - committed memory
		MemoryStats.AvailableVirtual = static_cast<uint64>(static_cast<float>(MemTotal + SwapTotal)	* (1.0f + OvercommitRatio)) - CommittedAS;
	}

	FProcField SMapsFields2[] =
	{
		{ "VmPeak:", &MemoryStats.PeakUsedVirtual },
		{ "VmSize:", &MemoryStats.UsedVirtual },      // In /proc/self/statm (Field 1)
		{ "VmHWM:",  &MemoryStats.PeakUsedPhysical },
		{ "VmRSS:",  &MemoryStats.UsedPhysical },     // In /proc/self/statm (Field 2)
	};
	if (ReadProcFields("/proc/self/status", SMapsFields2, UE_ARRAY_COUNT(SMapsFields2)))
	{
		// sanitize stats as sometimes peak < used for some reason
		MemoryStats.PeakUsedVirtual = FMath::Max(MemoryStats.PeakUsedVirtual, MemoryStats.UsedVirtual);
		MemoryStats.PeakUsedPhysical = FMath::Max(MemoryStats.PeakUsedPhysical, MemoryStats.UsedPhysical);
	}

	return MemoryStats;
}

static uint64 ParseSMapsFileChunk(ANSICHAR *Buffer, uint64 BufferSize, FProcField *ProcFields, uint32 NumFields)
{
	uint64 ParsePos = 0;

	for (uint64 Idx = 0; Idx < BufferSize; Idx++)
	{
		if (Buffer[Idx] == '\n')
		{
			const ANSICHAR *Line = Buffer + ParsePos;

			Buffer[Idx] = 0;

			for (uint32 IdxField = 0; IdxField < NumFields; IdxField++)
			{
				uint32 NameLen = ProcFields[IdxField].NameLen;

				if (!FCStringAnsi::Strncmp(Line, ProcFields[IdxField].Name, NameLen))
				{
					*ProcFields[IdxField].Addr += atoll(Line + NameLen) * 1024ULL;
					break;
				}
			}

			ParsePos = Idx + 1;
		}
	}

	// If we didn't find any linefeeds, skip the entire chunk
	return ParsePos ? ParsePos : BufferSize;
}

// For tracking what we've seen in the smaps output between addresses
struct FPageParseState
{
	bool bHavePageAddress = false;
	bool bHaveSharedClean = false;
	bool bHaveSharedDirty = false;
	bool bHavePrivateClean = false;
	bool bHavePrivateDirty = false;
	bool Complete()
	{
		return bHaveSharedClean && bHaveSharedDirty && bHavePrivateClean && bHavePrivateDirty;
	}
	void Reset()
	{
		*this = FPageParseState();
	}
};

// info on where to put the smaps field and which ones we wants.
struct FProcFieldWithOffset
{
	const ANSICHAR* Name = nullptr;
	uint64 Offset = 0;
	uint64 ConfirmOffset = 0;
	uint32 NameLen = 0;

	FProcFieldWithOffset(const ANSICHAR* NameIn, uint64 OffsetIn, uint64 ConfirmOffsetIn)
		: Name(NameIn), Offset(OffsetIn), ConfirmOffset(ConfirmOffsetIn)
	{
		NameLen = FCStringAnsi::Strlen(Name);
	}
};

// Look in Buffer for a line and advance it, parsing out data that we care about. Returns how far in to Buffer we made it before
// we couldn't get a complete line.
static uint64 ParseSMapsPage(ANSICHAR* Buffer, uint64 BufferSize, FPageParseState& State, TArray<FForkedPageAllocation>& Pages, FProcFieldWithOffset* Fields, uint32 FieldCount)
{
	uint64 ParsePos = 0;

	for (uint64 Idx = 0; Idx < BufferSize; Idx++)
	{
		if (Buffer[Idx] == '\n')
		{
			ANSICHAR* Line = Buffer + ParsePos;

			Buffer[Idx] = 0;

			// If the line starts with a number, its an address.
			if ((Line[0] >= '0' && Line[0] <= '9') ||
				(Line[0] >= 'a' && Line[0] <= 'f')) // all the fields start with capitals so this works
			{
				check(!State.bHavePageAddress);

				// Start a new page.
				FForkedPageAllocation& Page = Pages.AddZeroed_GetRef();

				// 'from' address is up to '-'
				ANSICHAR* Dash = FCStringAnsi::Strchr(Line, '-');
				ANSICHAR* Space = FCStringAnsi::Strchr(Dash, ' ');

				Dash[0] = 0;
				Space[0] = 0;

				Page.PageStart = strtoll(Line, 0, 16);
				Page.PageEnd = strtoll(Dash + 1, 0, 16);

				State.bHavePageAddress = true;
			}
			else if (State.bHavePageAddress)
			{
				for (uint32 IdxField = 0; IdxField < FieldCount; IdxField++)
				{
					uint32 NameLen = Fields[IdxField].NameLen;

					if (!FCStringAnsi::Strncmp(Line, Fields[IdxField].Name, NameLen))
					{
						// Make sure we don't already have it
						check(((bool*)(&State))[Fields[IdxField].ConfirmOffset] == false);
						((bool*)(&State))[Fields[IdxField].ConfirmOffset] = true;

						uint64 ResultKb = atoll(Line + NameLen);

						FForkedPageAllocation* Page = &Pages.Top();

						((uint64*)Page)[Fields[IdxField].Offset] = ResultKb;

						if (State.Complete())
						{
							// If we have all the fields we care about, go back to looking for an address.
							State.Reset();
						}
						break;
					}
				}
			}

			ParsePos = Idx + 1;
		}
	}

	// If we didn't find any linefeeds, skip the entire chunk
	return ParsePos ? ParsePos : BufferSize;
}

bool FUnixPlatformMemory::GetForkedPageAllocationInfo(TArray<FForkedPageAllocation>& OutPages)
{
	OutPages.Reset();

	const ANSICHAR Shared_CleanStr[] = "Shared_Clean:";
	const ANSICHAR Shared_DirtyStr[] = "Shared_Dirty:";
	const ANSICHAR Private_CleanStr[] = "Private_Clean:";
	const ANSICHAR Private_DirtyStr[] = "Private_Dirty:";


	FProcFieldWithOffset SMapsFields[] =
	{
		{ Shared_CleanStr, offsetof(FForkedPageAllocation, SharedCleanKiB) / 8 , offsetof(FPageParseState, bHaveSharedClean)    },
		{ Shared_DirtyStr, offsetof(FForkedPageAllocation, SharedDirtyKiB) / 8 , offsetof(FPageParseState, bHaveSharedDirty)    },
		{ Private_CleanStr,offsetof(FForkedPageAllocation, PrivateCleanKiB) / 8, offsetof(FPageParseState, bHavePrivateClean)  },
		{ Private_DirtyStr,offsetof(FForkedPageAllocation, PrivateDirtyKiB) / 8, offsetof(FPageParseState, bHavePrivateDirty)  },
	};

	int Fd = open("/proc/self/smaps", O_RDONLY);
	if (Fd < 0)
	{
		UE_LOGF(LogHAL, Warning, "Failed to open /proc/self/smaps for pages");
		return false;
	}

	const uint32 ChunkSize = 16*1024;
	ANSICHAR Buffer[ChunkSize];
	uint64 BytesAvailableInChunk = 0;

	FPageParseState ParseState;

	for (;;)
	{
		ssize_t BytesRead;
		uint64 BytesToRead = ChunkSize - BytesAvailableInChunk;

		do
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_GetPages_FileRead);
			BytesRead = read(Fd, Buffer + BytesAvailableInChunk, BytesToRead);
		} while (BytesRead < 0 && errno == EINTR);

		if (BytesRead <= 0)
		{
			break;
		}

		BytesAvailableInChunk += BytesRead;

		uint64 BytesParsed = ParseSMapsPage(Buffer, BytesAvailableInChunk, ParseState, OutPages, SMapsFields, UE_ARRAY_COUNT(SMapsFields));
		checkf(BytesParsed <= BytesAvailableInChunk, TEXT("BytesParsed more than BytesAvailableInChunk %llu %llu"), BytesParsed, BytesAvailableInChunk);
		if (BytesParsed > BytesAvailableInChunk)
		{
			FPlatformMisc::LowLevelOutputDebugString(TEXT("Critical parse fail in ParseSMapsPage"));
			close(Fd);
			return false;
		}

		BytesAvailableInChunk -= BytesParsed;
		memmove(Buffer, Buffer + BytesParsed, BytesAvailableInChunk);

		// If we made no parsing progress and we've reached end of file, stop trying.
		// This is a sanity check because we expect to complete parsing prior to getting EOF, naturally exiting
		// in the above <= 0 check.
		if (BytesRead == 0 &&
			BytesParsed == 0)
		{
			break;
		}
	}

	close(Fd);
	return true;
}



FExtendedPlatformMemoryStats FUnixPlatformMemory::GetExtendedStats()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_GetExtendedStats);
	
	const ANSICHAR Shared_CleanStr[]  = "Shared_Clean:";
	const ANSICHAR Shared_DirtyStr[]  = "Shared_Dirty:";
	const ANSICHAR Private_CleanStr[] = "Private_Clean:";
	const ANSICHAR Private_DirtyStr[] = "Private_Dirty:";

	FExtendedPlatformMemoryStats MemoryStats = { 0 };

	FProcField SMapsFields[] =
	{
		{ Shared_CleanStr,  (uint64 *)&MemoryStats.Shared_Clean },
		{ Shared_DirtyStr,  (uint64 *)&MemoryStats.Shared_Dirty },
		{ Private_CleanStr, (uint64 *)&MemoryStats.Private_Clean },
		{ Private_DirtyStr, (uint64 *)&MemoryStats.Private_Dirty },
	};

#if USE_PROC_SELF_SMAPS_ROLLUP
	// ~ 1.06ms per call on my Threadripper 3990X w/ Debian Testing 5.15.0-2-amd64
	// Note that testing shows us opening smaps_rollup on a thread while another thread is
	//  calling mmap() can cause large spikes (30+ms) in the mmap.
	if (!ReadProcFields("/proc/self/smaps_rollup", SMapsFields, UE_ARRAY_COUNT(SMapsFields)))
#endif
	{
		int Fd = open("/proc/self/smaps", O_RDONLY);

		if (Fd >= 0)
		{
			const uint32 ChunkSize = 512;
			ANSICHAR Buffer[ChunkSize];
			uint64 BytesAvailableInChunk = 0;

			for (;;)
			{
				ssize_t BytesRead;
				uint64 BytesToRead = ChunkSize - BytesAvailableInChunk;

				do
				{
					QUICK_SCOPE_CYCLE_COUNTER(STAT_GetExtendedStats_FileRead);
					BytesRead = read(Fd, Buffer + BytesAvailableInChunk, BytesToRead);
				} while (BytesRead < 0 && errno == EINTR);

				if (BytesRead <= 0)
				{
					break;
				}

				BytesAvailableInChunk += BytesRead;

				uint64 BytesParsed = ParseSMapsFileChunk(Buffer, BytesAvailableInChunk, SMapsFields, UE_ARRAY_COUNT(SMapsFields));
				checkf(BytesParsed <= BytesAvailableInChunk, TEXT("BytesParsed more than BytesAvailableInChunk %llu %llu"), BytesParsed, BytesAvailableInChunk);
				if (BytesParsed > BytesAvailableInChunk)
				{
					FPlatformMisc::LowLevelOutputDebugString(TEXT("Critical parse fail in ParseSMapsPage"));
					close(Fd);
					return FExtendedPlatformMemoryStats();
				}

				BytesAvailableInChunk -= BytesParsed;
				memmove(Buffer, Buffer + BytesParsed, BytesAvailableInChunk);
			}

			close(Fd);
		}
	}

	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_GetExtendedStats_KsmStat);
		
		int Fd = open("/sys/kernel/mm/ksm/run", O_RDONLY);
		if (Fd != -1)
		{
			char Value = 0;
			read(Fd, &Value, 1);
			close(Fd);

			if (Value == '1')
			{
				MemoryStats.bIs_KSM_Enabled = true;

				FProcField Field = { "ksm_process_profit", (uint64 *)&MemoryStats.KSM_Process_Profit };
				if (ReadProcFields("/proc/self/ksm_stat", &Field, 1))
				{
					// ReadProcFields expects KiB so has already multiplied by 1024
					MemoryStats.KSM_Process_Profit /= 1024;
				}
			}
		}
	}

	MemoryStats.Shared_Memory_COW_Bytes = GSharedMemoryCOWBytes;

	return MemoryStats;
}

const FPlatformMemoryConstants& FUnixPlatformMemory::GetConstants()
{
	static FPlatformMemoryConstants MemoryConstants;

	if( MemoryConstants.TotalPhysical == 0 )
	{
		// Gather platform memory stats.
		struct sysinfo SysInfo;
		unsigned long long MaxPhysicalRAMBytes = 0;
		unsigned long long MaxVirtualRAMBytes = 0;

		if (0 == sysinfo(&SysInfo))
		{
			MaxPhysicalRAMBytes = static_cast< unsigned long long >( SysInfo.mem_unit ) * static_cast< unsigned long long >( SysInfo.totalram );
			MaxVirtualRAMBytes
				= MaxPhysicalRAMBytes
				+ static_cast< unsigned long long >( SysInfo.mem_unit ) * static_cast< unsigned long long >( SysInfo.totalswap );
		}

		MemoryConstants.TotalPhysical = MaxPhysicalRAMBytes;
		MemoryConstants.TotalVirtual = MaxVirtualRAMBytes;

		MemoryConstants.PageSize = sysconf(_SC_PAGESIZE);
		MemoryConstants.BinnedPageSize = FMath::Max((SIZE_T)65536, MemoryConstants.PageSize);
		MemoryConstants.BinnedAllocationGranularity = MemoryConstants.PageSize;
		MemoryConstants.OsAllocationGranularity = MemoryConstants.PageSize;

		MemoryConstants.TotalPhysicalGB = (MemoryConstants.TotalPhysical + 1024ULL * 1024ULL * 1024ULL - 1) / 1024ULL / 1024ULL / 1024ULL;
		MemoryConstants.AddressLimit = FPlatformMath::RoundUpToPowerOfTwo64(MemoryConstants.TotalPhysical);
	}

	return MemoryConstants;	
}

FPlatformMemory::FSharedMemoryRegion* FUnixPlatformMemory::MapNamedSharedMemoryRegion(const FString& InName, bool bCreate, uint32 AccessMode, SIZE_T Size)
{
	// expecting platform-independent name, so convert it to match platform requirements
	FString Name("/");
	Name += InName;
	FTCHARToUTF8 NameUTF8(*Name);

	// correct size to match platform constraints
	FPlatformMemoryConstants MemConstants = FPlatformMemory::GetConstants();
	check(MemConstants.PageSize > 0);	// also relying on it being power of two, which should be true in foreseeable future
	if (Size & (MemConstants.PageSize - 1))
	{
		Size = Size & ~(MemConstants.PageSize - 1);
		Size += MemConstants.PageSize;
	}

	int ShmOpenFlags = bCreate ? O_CREAT : 0;
	// note that you cannot combine O_RDONLY and O_WRONLY to get O_RDWR
	check(AccessMode != 0);
	if (AccessMode == FPlatformMemory::ESharedMemoryAccess::Read)
	{
		ShmOpenFlags |= O_RDONLY;
	}
	else if (AccessMode == FPlatformMemory::ESharedMemoryAccess::Write)
	{
		ShmOpenFlags |= O_WRONLY;
	}
	else if (AccessMode == (FPlatformMemory::ESharedMemoryAccess::Write | FPlatformMemory::ESharedMemoryAccess::Read))
	{
		ShmOpenFlags |= O_RDWR;
	}

	int ShmOpenMode = (S_IRUSR | S_IWUSR) | (S_IRGRP | S_IWGRP) | (S_IROTH | S_IWOTH );	// 0666

	// open the object
	int SharedMemoryFd = shm_open(NameUTF8.Get(), ShmOpenFlags, ShmOpenMode);
	if (SharedMemoryFd == -1)
	{
		int ErrNo = errno;
		UE_LOGF(LogHAL, Warning, "shm_open(name='%ls', flags=0x%x, mode=0x%x) failed with errno = %d (%ls)", *Name, ShmOpenFlags, ShmOpenMode, ErrNo, 
			StringCast< TCHAR >(strerror(ErrNo)).Get());
		return NULL;
	}

	// truncate if creating (note that we may still don't have rights to do so)
	if (bCreate)
	{
		int Res = ftruncate(SharedMemoryFd, Size);
		if (Res != 0)
		{
			int ErrNo = errno;
			UE_LOGF(LogHAL, Warning, "ftruncate(fd=%d, size=%zu) failed with errno = %d (%ls)", SharedMemoryFd, Size, ErrNo,
				StringCast< TCHAR >(strerror(ErrNo)).Get());
			shm_unlink(NameUTF8.Get());
			return NULL;
		}
	}

	// map
	int MmapProtFlags = 0;
	if (AccessMode & FPlatformMemory::ESharedMemoryAccess::Read)
	{
		MmapProtFlags |= PROT_READ;
	}

	if (AccessMode & FPlatformMemory::ESharedMemoryAccess::Write)
	{
		MmapProtFlags |= PROT_WRITE;
	}

	void *Ptr = mmap(NULL, Size, MmapProtFlags, MAP_SHARED, SharedMemoryFd, 0);
	if (Ptr == MAP_FAILED)
	{
		int ErrNo = errno;
		UE_LOGF(LogHAL, Warning, "mmap(addr=NULL, length=%zu, prot=0x%x, flags=MAP_SHARED, fd=%d, 0) failed with errno = %d (%ls)", Size, MmapProtFlags, SharedMemoryFd, ErrNo,
			StringCast< TCHAR >(strerror(ErrNo)).Get());

		if (bCreate)
		{
			shm_unlink(NameUTF8.Get());
		}
		return NULL;
	}

	return new FUnixSharedMemoryRegion(Name, AccessMode, Ptr, Size, SharedMemoryFd, bCreate);
}

bool FUnixPlatformMemory::UnmapNamedSharedMemoryRegion(FSharedMemoryRegion * MemoryRegion)
{
	bool bAllSucceeded = true;

	if (MemoryRegion)
	{
		FUnixSharedMemoryRegion * UnixRegion = static_cast< FUnixSharedMemoryRegion* >( MemoryRegion );

		if (munmap(UnixRegion->GetAddress(), UnixRegion->GetSize()) == -1) 
		{
			bAllSucceeded = false;

			int ErrNo = errno;
			UE_LOGF(LogHAL, Warning, "munmap(addr=%p, len=%zu) failed with errno = %d (%ls)", UnixRegion->GetAddress(), UnixRegion->GetSize(), ErrNo,
				StringCast< TCHAR >(strerror(ErrNo)).Get());
		}

		if (close(UnixRegion->GetFileDescriptor()) == -1)
		{
			bAllSucceeded = false;

			int ErrNo = errno;
			UE_LOGF(LogHAL, Warning, "close(fd=%d) failed with errno = %d (%ls)", UnixRegion->GetFileDescriptor(), ErrNo, 
				StringCast< TCHAR >(strerror(ErrNo)).Get());
		}

		if (UnixRegion->NeedsToUnlinkRegion())
		{
			FTCHARToUTF8 NameUTF8(UnixRegion->GetName());
			if (shm_unlink(NameUTF8.Get()) == -1)
			{
				bAllSucceeded = false;

				int ErrNo = errno;
				UE_LOGF(LogHAL, Warning, "shm_unlink(name='%ls') failed with errno = %d (%ls)", UnixRegion->GetName(), ErrNo, 
					StringCast< TCHAR >(strerror(ErrNo)).Get());
			}
		}

		// delete the region
		delete UnixRegion;
	}

	return bAllSucceeded;
}

void FUnixPlatformMemory::OnOutOfMemory(uint64 Size, uint32 Alignment)
{
	auto HandleOOM = [&]()
	{
		// Update memory stats before we enter the crash handler.
		OOMAllocationSize = Size;
		OOMAllocationAlignment = Alignment;

		bIsOOM = true;

		const int ErrorMsgSize = 256;
		TCHAR ErrorMsg[ErrorMsgSize];
		FPlatformMisc::GetSystemErrorMessage(ErrorMsg, ErrorMsgSize, 0);

		FMalloc* Prev = UE::Private::GMalloc;
		FPlatformMallocCrash::Get().SetAsGMalloc();

		FPlatformMemoryStats PlatformMemoryStats = FPlatformMemory::GetStats();

		UE_LOG(LogMemory, Warning, TEXT("MemoryStats:")\
			TEXT("\n\tAvailablePhysical %llu")\
			TEXT("\n\t AvailableVirtual %llu")\
			TEXT("\n\t     UsedPhysical %llu")\
			TEXT("\n\t PeakUsedPhysical %llu")\
			TEXT("\n\t      UsedVirtual %llu")\
			TEXT("\n\t  PeakUsedVirtual %llu"),
			(uint64)PlatformMemoryStats.AvailablePhysical,
			(uint64)PlatformMemoryStats.AvailableVirtual,
			(uint64)PlatformMemoryStats.UsedPhysical,
			(uint64)PlatformMemoryStats.PeakUsedPhysical,
			(uint64)PlatformMemoryStats.UsedVirtual,
			(uint64)PlatformMemoryStats.PeakUsedVirtual);
		if (GWarn)
		{
			Prev->DumpAllocatorStats(*GWarn);
		}

		// let any registered handlers go
		FCoreDelegates::GetOutOfMemoryDelegate().Broadcast();

		// ErrorMsg might be unrelated to OoM error in some cases as the code that calls OnOutOfMemory could have called other system functions that modified errno
		UE_LOGF(LogMemory, Fatal, "Ran out of memory allocating %llu bytes with alignment %u. Last error msg: %ls.", Size, Alignment, ErrorMsg);
	};
	
	UE_CALL_ONCE(HandleOOM);
	FPlatformProcess::SleepInfinite(); // Unreachable
}

/**
* LLM uses these low level functions (LLMAlloc and LLMFree) to allocate memory. It grabs
* the function pointers by calling FPlatformMemory::GetLLMAllocFunctions. If these functions
* are not implemented GetLLMAllocFunctions should return false and LLM will be disabled.
*/

#if ENABLE_LOW_LEVEL_MEM_TRACKER

void* LLMAlloc(size_t Size)
{
	void* Ptr = mmap(nullptr, Size, PROT_READ | PROT_WRITE, MAP_PRIVATE | MAP_ANON, -1, 0);

	return Ptr;
}

void LLMFree(void* Addr, size_t Size)
{
	if (Addr != nullptr && munmap(Addr, Size) != 0)
	{
		const int ErrNo = errno;
		UE_LOGF(LogHAL, Fatal, "munmap(addr=%p, len=%llu) failed with errno = %d (%ls)", Addr, Size,
			ErrNo, StringCast< TCHAR >(strerror(ErrNo)).Get());
	}
}

#endif

bool FUnixPlatformMemory::GetLLMAllocFunctions(void*(*&OutAllocFunction)(size_t), void(*&OutFreeFunction)(void*, size_t), int32& OutAlignment)
{
#if ENABLE_LOW_LEVEL_MEM_TRACKER
	OutAllocFunction = LLMAlloc;
	OutFreeFunction = LLMFree;
	OutAlignment = FPlatformMemory::GetConstants().PageSize;
	return true;
#else
	return false;
#endif
}

bool FUnixPlatformMemory::CanOverallocateVirtualMemory()
{
	static int OvercommitPolicy = ReadOvercommitMode();
	return OvercommitPolicy != 2;
}

void FUnixPlatformMemory::IncSharedMemoryCOWBytes(size_t Bytes)
{
	std::atomic_fetch_add(&GSharedMemoryCOWBytes, Bytes);
}
