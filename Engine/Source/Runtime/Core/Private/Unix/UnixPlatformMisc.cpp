// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GenericPlatformMisc.cpp: Generic implementations of misc platform functions
=============================================================================*/

#include "Unix/UnixPlatformMisc.h"
#include "Misc/AssertionMacros.h"
#include "GenericPlatform/GenericPlatformMemory.h"
#include "HAL/UnrealMemory.h"
#include "Templates/UnrealTemplate.h"
#include "Math/UnrealMathUtility.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Misc/DateTime.h"
#include "Misc/StringBuilder.h"
#include "HAL/PlatformTime.h"
#include "Containers/StringConv.h"
#include "Logging/LogMacros.h"
#include "Misc/Parse.h"
#include "Misc/CommandLine.h"
#include "Misc/App.h"
#include "Misc/FileHelper.h"
#include "Unix/UnixPlatformCrashContext.h"
#include "Unix/UnixPlatformGuid.h"
#include "Unix/UnixPlatformSyscallTable.h"
#include "Misc/ConfigCacheIni.h"
#include "GenericPlatform/GenericPlatformChunkInstall.h"
#include "Misc/OutputDeviceRedirector.h"

#if PLATFORM_HAS_CPUID
	#include <cpuid.h>
#endif // PLATFORM_HAS_CPUID
#include <sys/sysinfo.h>
#include <sys/utsname.h>
#include <sched.h>
#include <sys/vfs.h>
#include <sys/ioctl.h>

#include <sys/prctl.h>
#include <linux/audit.h>
#include <linux/filter.h>
#include <linux/seccomp.h>

#include <ifaddrs.h>
#include <net/if.h>
#include <net/if_arp.h>
#include <linux/limits.h>

#include <glob.h>

#include "Modules/ModuleManager.h"
#include "HAL/ThreadHeartBeat.h"

#include "FramePro/FrameProProfiler.h"
#include "BuildSettings.h"

extern bool GInitializedSDL;

namespace PlatformMiscLimits
{
	enum
	{
		MaxOsGuidLength = 32
	};
};

namespace
{
	/**
	 * Empty handler so some signals are just not ignored
	 */
	void EmptyChildHandler(int32 Signal, siginfo_t* Info, void* Context)
	{
	}

	/**
	 * Installs SIGCHLD signal handler so we can wait for our children (otherwise they are reaped automatically)
	 */
	void InstallChildExitedSignalHanlder()
	{
		struct sigaction Action;
		FMemory::Memzero(Action);
		Action.sa_sigaction = EmptyChildHandler;
		sigfillset(&Action.sa_mask);
		Action.sa_flags = SA_SIGINFO | SA_RESTART | SA_ONSTACK;
		sigaction(SIGCHLD, &Action, nullptr);
	}
}

void FUnixPlatformMisc::NormalizePath(FString& InPath)
{
	// only expand if path starts with ~, e.g. ~/ should be expanded, /~ should not
	if (InPath.StartsWith(TEXT("~"), ESearchCase::CaseSensitive))	// case sensitive is quicker, and our substring doesn't care
	{
		InPath = InPath.Replace(TEXT("~"), FPlatformProcess::UserHomeDir(), ESearchCase::CaseSensitive);
	}
}

void FUnixPlatformMisc::NormalizePath(FStringBuilderBase& InPath)
{
	// only expand if path starts with ~, e.g. ~/ should be expanded, /~ should not
	if (FStringView(InPath).StartsWith('~'))
	{
		InPath.ReplaceAt(0, 1, FPlatformProcess::UserHomeDir());
	}
}

// Defined in UnixPlatformMemory
extern bool GUseKSM;
extern bool GKSMMergeAllPages;
extern uint64 GCrashHandlerStackSize;

static void UnixPlatForm_CheckIfKSMUsable()
{
	// https://www.kernel.org/doc/Documentation/vm/ksm.txt
	if (GUseKSM)
	{
		int KSMRunEnabled = 0;
		if (FILE* KSMRunFile = fopen("/sys/kernel/mm/ksm/run", "r"))
		{
			if (fscanf(KSMRunFile, "%d", &KSMRunEnabled) != 1)
			{
				KSMRunEnabled = 0;
			}

			fclose(KSMRunFile);
		}

		// The range for PagesToScan is 0 <--> max uint32_t
		uint32_t PagesToScan = 0;
		if (FILE* KSMPagesToScanFile = fopen("/sys/kernel/mm/ksm/pages_to_scan", "r"))
		{
			if (fscanf(KSMPagesToScanFile, "%u", &PagesToScan) != 1)
			{
				PagesToScan = 0;
			}

			fclose(KSMPagesToScanFile);
		}

		if (!KSMRunEnabled)
		{
			GUseKSM = 0;
			UE_LOGF(LogInit, Error, "Cannot run ksm when its disabled in the kernel. Please check /sys/kernel/mm/ksm/run");
		}
		else
		{
			if (PagesToScan <= 0)
			{
				GUseKSM = 0;
				UE_LOGF(LogInit, Error, "KSM enabled but number of pages to be scanned is 0 which will implicitly disable KSM. Please check /sys/kernel/mm/ksm/pages_to_scan");
			}
			else
			{
				UE_LOGF(LogInit, Log, "KSM enabled. Number of pages to be scanned before ksmd goes to sleep: %u", PagesToScan);
			}
		}
	}

	// Disable if GUseKSM is disabled from kernel settings
	GKSMMergeAllPages = GUseKSM && GKSMMergeAllPages;
}

// Init'ed in UnixPlatformMemory for now. Once the old crash symbolicator is gone remove this
extern bool CORE_API GUseNewCrashSymbolicator;

// Function used to read and store the entire *.sym file associated with the main module in memory.
// Helps greatly by reducing I/O during ensures/crashes. Only suggested when running a monolithic build
extern void CORE_API UnixPlatformStackWalk_PreloadModuleSymbolFile();

void FUnixPlatformMisc::PlatformPreInit()
{
	FGenericPlatformMisc::PlatformPreInit();

	UnixCrashReporterTracker::PreInit();
}

void FUnixPlatformMisc::PlatformInit()
{
	// install a platform-specific signal handler
	InstallChildExitedSignalHanlder();

	// do not remove the below check for IsFirstInstance() - it is not just for logging, it actually lays the claim to be first
	bool bFirstInstance = FPlatformProcess::IsFirstInstance();
	bool bIsNullRHI = !FApp::CanEverRender();

	bool bPreloadedModuleSymbolFile = FParse::Param(FCommandLine::Get(), TEXT("preloadmodulesymbols"));

	UnixPlatForm_CheckIfKSMUsable();

	FString GPUBrandInfo = GetPrimaryGPUBrand();

	UE_LOGF(LogInit, Log, "Unix hardware info:");
	UE_LOGF(LogInit, Log, " - we are %lsthe first instance of this executable", bFirstInstance ? TEXT("") : TEXT("not "));
	UE_LOGF(LogInit, Log, " - this process' id (pid) is %d, parent process' id (ppid) is %d", static_cast< int32 >(getpid()), static_cast< int32 >(getppid()));
	UE_LOGF(LogInit, Log, " - we are %lsrunning under debugger", IsDebuggerPresent() ? TEXT("") : TEXT("not "));
	UE_LOGF(LogInit, Log, " - machine network name is '%ls'", FPlatformProcess::ComputerName());
	UE_LOGF(LogInit, Log, " - user name is '%ls' (%ls)", FPlatformProcess::UserName(), FPlatformProcess::UserName(false));
	UE_LOGF(LogInit, Log, " - we're logged in %ls", FPlatformMisc::HasBeenStartedRemotely() ? TEXT("remotely") : TEXT("locally"));
	UE_LOGF(LogInit, Log, " - we're running %ls rendering", bIsNullRHI ? TEXT("without") : TEXT("with"));
	UE_LOGF(LogInit, Log, " - CPU: %ls '%ls' (signature: 0x%X)", *FPlatformMisc::GetCPUVendor(), *FPlatformMisc::GetCPUBrand(), FPlatformMisc::GetCPUInfo());
	UE_LOGF(LogInit, Log, " - Number of physical cores available for the process: %d", FPlatformMisc::NumberOfCores());
	UE_LOGF(LogInit, Log, " - Number of logical cores available for the process: %d", FPlatformMisc::NumberOfCoresIncludingHyperthreads());

	if (!GPUBrandInfo.IsEmpty())
	{
		UE_LOGF(LogInit, Log, " - GPU Brand Info: %ls", *GPUBrandInfo);
	}

	UE_LOGF(LogInit, Log, " - Memory allocator used: %ls", UE::Private::GMalloc->GetDescriptiveName());
	UE_LOGF(LogInit, Log, " - This is %ls build.", BuildSettings::IsLicenseeVersion() ? TEXT("a licensee") : TEXT("an internal"));

	FPlatformTime::PrintCalibrationLog();

	UE_LOGF(LogInit, Log, "Unix-specific commandline switches:");
	UE_LOGF(LogInit, Log, " -ansimalloc - use malloc()/free() from libc (useful for tools like valgrind and electric fence)");
	UE_LOGF(LogInit, Log, " -jemalloc - use jemalloc for all memory allocation");
	UE_LOGF(LogInit, Log, " -binnedmalloc - use binned malloc  for all memory allocation");
	UE_LOGF(LogInit, Log, " -filemapcachesize=NUMBER - set the size for case-sensitive file mapping cache");
	UE_LOGF(LogInit, Log, " -useksm - uses kernel same-page mapping (KSM) for mapped memory (%ls)", GUseKSM ? TEXT("ON") : TEXT("OFF"));
	UE_LOGF(LogInit, Log, " -ksmmergeall - marks all mmap'd memory pages suitable for KSM (%ls)", GKSMMergeAllPages ? TEXT("ON") : TEXT("OFF"));
	UE_LOGF(LogInit, Log, " -preloadmodulesymbols - Loads the main module symbols file into memory (%ls)", bPreloadedModuleSymbolFile ? TEXT("ON") : TEXT("OFF"));
	UE_LOGF(LogInit, Log, " -sigdfl=SIGNAL - Allows a specific signal to be set to its default handler rather then ignoring the signal");
	UE_LOGF(LogInit, Log, " -crashhandlerstacksize - Allows setting crash handler stack sizes (%lu)", GCrashHandlerStackSize);
	UE_LOGF(LogInit, Log, " -noexclusivelockonwrite - disables marking files created by the engine as exclusive locked while the engine has them opened");

	// [RCL] FIXME: this should be printed in specific modules, if at all
	UE_LOGF(LogInit, Log, " -httpproxy=ADDRESS:PORT - redirects HTTP requests to a proxy (only supported if compiled with libcurl)");
	UE_LOGF(LogInit, Log, " -reuseconn - allow libcurl to reuse HTTP connections (only matters if compiled with libcurl)");
	UE_LOGF(LogInit, Log, " -virtmemkb=NUMBER - sets process virtual memory (address space) limit (overrides VirtualMemoryLimitInKB value from .ini)");
	UE_LOGF(LogInit, Log, " -allowsyscallfilterfile=PATH_TO_FILE - sets up a system call filter allow list. any invalid syscall in this list *will* cause a crash");

	if (bPreloadedModuleSymbolFile)
	{
		UnixPlatformStackWalk_PreloadModuleSymbolFile();
	}

	if (FPlatformMisc::HasBeenStartedRemotely() || FPlatformMisc::IsDebuggerPresent())
	{
		// print output immediately
		setvbuf(stdout, NULL, _IONBF, 0);
	}

	// This symbol is used for debugging but with LTO enabled it gets stripped as nothing is using it
	// Lets use it here just to log if its valid under VeryVerbose settings
	extern uint8** GNameBlocksDebug;
	if (GNameBlocksDebug)
	{
		UE_LOGF(LogInit, VeryVerbose, "GNameBlocksDebug Valid - %i", !!GNameBlocksDebug);
	}
}

extern void CORE_API UnixPlatformStackWalk_UnloadPreloadedModuleSymbol();
volatile sig_atomic_t GDeferedExitLogging = 0;

void FUnixPlatformMisc::PlatformTearDown()
{
	// We requested to close from a signal so we couldnt print.
	if (GDeferedExitLogging)
	{
		uint8 OverriddenErrorLevel = 0;
		if (FPlatformMisc::HasOverriddenReturnCode(&OverriddenErrorLevel))
		{
			UE_LOGF(LogCore, Log, "FUnixPlatformMisc::RequestExit(bForce=false, ReturnCode=%d)", OverriddenErrorLevel);
		}
		else
		{
			UE_LOGF(LogCore, Log, "FUnixPlatformMisc::RequestExit(false)");
		}
	}

	UnixPlatformStackWalk_UnloadPreloadedModuleSymbol();
	FPlatformProcess::CeaseBeingFirstInstance();
}

int32 FUnixPlatformMisc::GetMaxPathLength()
{
	return PATH_MAX;
}

FString FUnixPlatformMisc::GetEnvironmentVariable(const TCHAR* InVariableName)
{
	FString VariableName = InVariableName;
	VariableName.ReplaceInline(TEXT("-"), TEXT("_"));
	ANSICHAR *AnsiResult = secure_getenv(TCHAR_TO_ANSI(*VariableName));
	if (AnsiResult)
	{
		return UTF8_TO_TCHAR(AnsiResult);
	}
	else
	{
		return FString();
	}
}

void FUnixPlatformMisc::SetEnvironmentVar(const TCHAR* InVariableName, const TCHAR* Value)
{
	FString VariableName = InVariableName;
	VariableName.ReplaceInline(TEXT("-"), TEXT("_"));
	if (Value == NULL || Value[0] == TEXT('\0'))
	{
		unsetenv(TCHAR_TO_ANSI(*VariableName));
	}
	else
	{
		setenv(TCHAR_TO_ANSI(*VariableName), TCHAR_TO_ANSI(Value), 1);
	}
}

void FUnixPlatformMisc::LowLevelOutputDebugString(const TCHAR *Message)
{
	static_assert(PLATFORM_USE_LS_SPEC_FOR_WIDECHAR, "Check printf format");
	fprintf(stderr, "%s", TCHAR_TO_UTF8(Message));	// there's no good way to implement that really
}

extern volatile sig_atomic_t GEnteredSignalHandler;
uint8 GOverriddenReturnCode = 0;
bool GHasOverriddenReturnCode = false;

void FUnixPlatformMisc::RequestExit(bool Force, const TCHAR* CallSite)
{
	if (GEnteredSignalHandler)
	{
		// Still log something but use a signal-safe function
		const ANSICHAR ExitMsg[] = "FUnixPlatformMisc::RequestExit\n";
		write(STDOUT_FILENO, ExitMsg, sizeof(ExitMsg));

		GDeferedExitLogging = 1;
	}
	else
	{
		UE_LOGF(LogCore, Log,  "FUnixPlatformMisc::RequestExit(%i, %ls)", Force,
			CallSite ? CallSite : TEXT("<NoCallSiteInfo>"));
	}

	if(Force)
	{
		// Make sure the log is flushed.
		if (!GEnteredSignalHandler && GLog)
		{
			GLog->Flush();
		}

		// Force immediate exit. Cannot call abort() here, because abort() raises SIGABRT which we treat as crash
		// (to prevent other, particularly third party libs, from quitting without us noticing)
		// Propagate override return code, but normally don't exit with 0, so the parent knows it wasn't a normal exit.
		if (GHasOverriddenReturnCode)
		{
			_exit(GOverriddenReturnCode);
		}
		else
		{
			_exit(1);
		}
	}

	if (GEnteredSignalHandler)
	{
		// Lets set our selfs to request exit as the generic platform request exit could log
		// This is deprecated but one of the few excpetions to leave around for now as we dont want to UE_LOG as that may allocate memory
		// ShouldRequestExit *should* only be used this way in cases where non-reentrant code is required
#if UE_SET_REQUEST_EXIT_ON_TICK_ONLY
		extern bool GShouldRequestExit;
		GShouldRequestExit = 1;
#else
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		GIsRequestingExit = 1;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // UE_SET_REQUEST_EXIT_ON_TICK_ONLY
	}
	else
	{
		// Tell the platform specific code we want to exit cleanly from the main loop.
		FGenericPlatformMisc::RequestExit(Force, CallSite);
	}
}

void FUnixPlatformMisc::RequestExitWithStatus(bool Force, uint8 ReturnCode, const TCHAR* CallSite)
{
	if (GEnteredSignalHandler)
	{
		// Still log something but use a signal-safe function
		const ANSICHAR ExitMsg[] = "FUnixPlatformMisc::RequestExitWithStatus\n";
		write(STDOUT_FILENO, ExitMsg, sizeof(ExitMsg));

		GDeferedExitLogging = 1;
	}
	else
	{
		UE_LOGF(LogCore, Log, "FUnixPlatformMisc::RequestExit(bForce=%ls, ReturnCode=%d, CallSite=%ls)",
			Force ? TEXT("true") : TEXT("false"), ReturnCode, CallSite ? CallSite : TEXT("<NoCallSiteInfo>"));
	}

	GOverriddenReturnCode = ReturnCode;
	GHasOverriddenReturnCode = true;

	return FPlatformMisc::RequestExit(Force, CallSite);
}

bool FUnixPlatformMisc::HasOverriddenReturnCode(uint8 * OverriddenReturnCodeToUsePtr)
{
	if (GHasOverriddenReturnCode && OverriddenReturnCodeToUsePtr != nullptr)
	{
		*OverriddenReturnCodeToUsePtr = GOverriddenReturnCode;
	}

	return GHasOverriddenReturnCode;
}

namespace
{
	/**
	 * Gets a Linux kernel version string
	 */
	FString GetKernelVersion()
	{
		struct utsname SysInfo;

		if (uname(&SysInfo) == 0)
		{
			return FString(SysInfo.release);
		}

		return FString();
	}

	bool IsRunningOnChromiumOS()
	{
		bool bRet = false;
		int SysVendorFile = open("/sys/class/dmi/id/sys_vendor", O_RDONLY);

		if (SysVendorFile >= 0)
		{
			char LineBuffer[128];
			ssize_t Length = read(SysVendorFile, LineBuffer, sizeof(LineBuffer));

			if (Length > 0)
			{
				bRet = !FCStringAnsi::Strncmp(LineBuffer, "ChromiumOS", 10);
			}

			close(SysVendorFile);
		}

		return bRet;
	}

	/**
	 * Reads a Linux style configuration file ignoring # as comments
	 */
	TMap<FString, FString> ReadConfigurationFile(const TCHAR* Filename)
	{
		TMap<FString, FString> Contents;
		TArray<FString> ConfigLines;
		if (FFileHelper::LoadANSITextFileToStrings(Filename, &IFileManager::Get(), ConfigLines))
		{
			for (const FString& Line : ConfigLines)
			{
				FString CleanLine = Line.TrimStartAndEnd();
				FString KeyString;
				FString ValueString;
				// Skip Comments
				if (!CleanLine.StartsWith("#"))
				{
					if (CleanLine.Split(TEXT("="), &KeyString, &ValueString, ESearchCase::CaseSensitive))
					{
						KeyString.TrimStartAndEndInline();
						ValueString.TrimStartAndEndInline();
						if (ValueString.Left(1) == TEXT("\""))
						{
							ValueString.RightChopInline(1, EAllowShrinking::No);
						}
						if (ValueString.Right(1) == TEXT("\""))
						{
							ValueString.LeftChopInline(1, EAllowShrinking::No);
						}
						Contents.Add(KeyString, ValueString);
					}
				}
			}
		}
		return Contents;
	}
}

void FUnixPlatformMisc::GetOSVersions(FString& out_OSVersionLabel, FString& out_OSSubVersionLabel)
{
	// Set up Fallback Details
	out_OSVersionLabel = FString("GenericLinuxVersion");
	// Get Kernel Version
	out_OSSubVersionLabel = GetKernelVersion();

	// Get PRETTY_NAME/NAME or redhat-release line
	TMap<FString, FString> OsInfo = ReadConfigurationFile(TEXT("/etc/os-release"));
	if (OsInfo.Num() > 0)
	{
		FString* VersionAddress = OsInfo.Find(TEXT("PRETTY_NAME"));
		if (VersionAddress)
		{
			FString* VersionNameAddress = nullptr;
			if (VersionAddress->Equals(TEXT("Linux")))
			{
				VersionNameAddress = OsInfo.Find(TEXT("NAME"));
				if (VersionNameAddress != nullptr)
				{
					VersionAddress = VersionNameAddress;
				}
			}

			out_OSVersionLabel = FString(*VersionAddress);
		}
	}
	else
	{
		TArray <FString> RedHatRelease;
		if (FFileHelper::LoadANSITextFileToStrings(TEXT("/etc/redhat-release"), &IFileManager::Get(), RedHatRelease))
		{
			if (RedHatRelease.IsValidIndex(0))
			{
				out_OSVersionLabel = RedHatRelease[0];
			}
		}
	}

	if (IsRunningOnChromiumOS())
	{
		out_OSVersionLabel += TEXT(" ChromiumOS");
	}
}

UE_AUTORTFM_ALWAYS_OPEN
FString FUnixPlatformMisc::GetOSVersion()
{
	TMap<FString, FString> OsInfo = ReadConfigurationFile(TEXT("/etc/os-release"));
	if (OsInfo.Num() > 0)
	{
		FString Version;
		FString* IDAddress = OsInfo.Find(TEXT("ID"));
		if (IDAddress != nullptr)
		{
			Version = FString(*IDAddress);
		}

		IDAddress = OsInfo.Find(TEXT("VERSION_ID"));
		if (IDAddress != nullptr)
		{
			Version += FString(*IDAddress);
		}
		else
		{
			Version += GetKernelVersion();
		}

		if (!Version.IsEmpty())
		{
			return Version;
		}
	}

	TArray<FString> RedHatRelease;
	if (FFileHelper::LoadANSITextFileToStrings(TEXT("/etc/redhat-release"), &IFileManager::Get(), RedHatRelease))
	{
		if (RedHatRelease.IsValidIndex(0))
		{
			return RedHatRelease[0];
		}
	}

	return GetKernelVersion();
}

const TCHAR* FUnixPlatformMisc::GetSystemErrorMessage(TCHAR* OutBuffer, int32 BufferCount, int32 Error)
{
	check(OutBuffer && BufferCount);
	if (Error == 0)
	{
		Error = errno;
	}

	FString Message = FString::Printf(TEXT("errno=%d (%s)"), Error, UTF8_TO_TCHAR(strerror(Error)));
	FCString::Strncpy(OutBuffer, *Message, BufferCount);

	return OutBuffer;
}

uint32 FUnixPlatformMisc::GetLastError()
{
	return (uint32)errno;
}

CORE_API TFunction<EAppReturnType::Type(EAppMsgType::Type MsgType, const TCHAR* Text, const TCHAR* Caption)> MessageBoxExtCallback;

EAppReturnType::Type FUnixPlatformMisc::MessageBoxExt(EAppMsgType::Type MsgType, const TCHAR* Text, const TCHAR* Caption)
{
	if(MessageBoxExtCallback)
	{
		return MessageBoxExtCallback(MsgType, Text, Caption);
	}
	else
	{
		return FGenericPlatformMisc::MessageBoxExt(MsgType, Text, Caption);
	}
}

int32 FUnixPlatformMisc::NumberOfCores()
{
	// WARNING: this function ignores edge cases like affinity mask changes (and even more fringe cases like CPUs going offline)
	// in the name of performance (higher level code calls NumberOfCores() way too often...)
	static int32 NumberOfCores = 0;
	if (NumberOfCores == 0)
	{
		if (FParse::Param(FCommandLine::Get(), TEXT("usehyperthreading")))
		{
			NumberOfCores = NumberOfCoresIncludingHyperthreads();
		}
		else
		{
			cpu_set_t AvailableCpusMask;
			CPU_ZERO(&AvailableCpusMask);

			if (0 != sched_getaffinity(0, sizeof(AvailableCpusMask), &AvailableCpusMask))
			{
				NumberOfCores = 1;	// we are running on something, right?
			}
			else
			{
				char FileNameBuffer[1024];
				struct CpuInfo
				{
					int Core;
					int Package;
				}
				CpuInfos[CPU_SETSIZE];

				FMemory::Memzero(CpuInfos);
				int MaxCoreId = 0;
				int MaxPackageId = 0;
				int NumCpusAvailable = 0;

				for(int32 CpuIdx = 0; CpuIdx < CPU_SETSIZE; ++CpuIdx)
				{
					if (CPU_ISSET(CpuIdx, &AvailableCpusMask))
					{
						++NumCpusAvailable;

						sprintf(FileNameBuffer, "/sys/devices/system/cpu/cpu%d/topology/core_id", CpuIdx);

						if (FILE* CoreIdFile = fopen(FileNameBuffer, "r"))
						{
							if (1 != fscanf(CoreIdFile, "%d", &CpuInfos[CpuIdx].Core))
							{
								CpuInfos[CpuIdx].Core = 0;
							}
							fclose(CoreIdFile);
						}

						sprintf(FileNameBuffer, "/sys/devices/system/cpu/cpu%d/topology/physical_package_id", CpuIdx);

						if (FILE* PackageIdFile = fopen(FileNameBuffer, "r"))
						{
							// physical_package_id can be -1 on embedded devices - treat all CPUs as separate in that case.
							if (1 != fscanf(PackageIdFile, "%d", &CpuInfos[CpuIdx].Package) || CpuInfos[CpuIdx].Package < 0)
							{
								CpuInfos[CpuIdx].Package = CpuInfos[CpuIdx].Core;
							}
							fclose(PackageIdFile);
						}

						MaxCoreId = FMath::Max(MaxCoreId, CpuInfos[CpuIdx].Core);
						MaxPackageId = FMath::Max(MaxPackageId, CpuInfos[CpuIdx].Package);
					}
				}

				int NumCores = MaxCoreId + 1;
				int NumPackages = MaxPackageId + 1;
				int NumPairs = NumPackages * NumCores;

				// AArch64 topology seems to be incompatible with the above assumptions, particularly, core_id can be all 0 while the cores themselves are obviously independent.
				// Check if num CPUs available to us is more than 2 per core (i.e. more than reasonable when hyperthreading is involved), and if so, don't trust the topology.
				if (2 * NumCores < NumCpusAvailable)
				{
					NumberOfCores = NumCpusAvailable;	// consider all CPUs to be separate
				}
				else
				{
					unsigned char * Pairs = reinterpret_cast<unsigned char *>(FMemory_Alloca(NumPairs * sizeof(unsigned char)));
					FMemory::Memzero(Pairs, NumPairs * sizeof(unsigned char));

					for (int32 CpuIdx = 0; CpuIdx < CPU_SETSIZE; ++CpuIdx)
					{
						if (CPU_ISSET(CpuIdx, &AvailableCpusMask))
						{
							Pairs[CpuInfos[CpuIdx].Package * NumCores + CpuInfos[CpuIdx].Core] = 1;
						}
					}

					for (int32 Idx = 0; Idx < NumPairs; ++Idx)
					{
						NumberOfCores += Pairs[Idx];
					}
				}
			}
		}

		int32 LimitCount = 32768;
		if (FCommandLine::IsInitialized() && FParse::Value(FCommandLine::Get(), TEXT("-corelimit="), LimitCount))
		{
			NumberOfCores = FMath::Min(NumberOfCores, LimitCount);
		}

		// never allow it to be less than 1, we are running on something
		NumberOfCores = FMath::Max(1, NumberOfCores);
	}

	return NumberOfCores;
}

int32 FUnixPlatformMisc::NumberOfCoresIncludingHyperthreads()
{
	// WARNING: this function ignores edge cases like affinity mask changes (and even more fringe cases like CPUs going offline)
	// in the name of performance (higher level code calls NumberOfCores() way too often...)
	static int32 NumCoreIds = 0;
	if (NumCoreIds == 0)
	{
		cpu_set_t AvailableCpusMask;
		CPU_ZERO(&AvailableCpusMask);

		if (0 != sched_getaffinity(0, sizeof(AvailableCpusMask), &AvailableCpusMask))
		{
			NumCoreIds = 1;	// we are running on something, right?
		}
		else
		{
			NumCoreIds = CPU_COUNT(&AvailableCpusMask);
		}

		int32 LimitCount = 32768;
		if (FCommandLine::IsInitialized() && FParse::Value(FCommandLine::Get(), TEXT("-corelimit="), LimitCount))
		{
			NumCoreIds = FMath::Min(NumCoreIds, LimitCount);
		}
	}

	return NumCoreIds;
}

const TCHAR* FUnixPlatformMisc::GetNullRHIShaderFormat()
{
	if (FParse::Param(FCommandLine::Get(), TEXT("sm5")))
	{
		return TEXT("SF_VULKAN_SM5");
	}
	else if (FParse::Param(FCommandLine::Get(), TEXT("sm6")))
	{
		return TEXT("SF_VULKAN_SM6");
	}
	else
	{
		// Limit to the highest SM supported by the project
		TArray<FString> TargetedShaderFormats;
		GConfig->GetArray(TEXT("/Script/LinuxTargetPlatform.LinuxTargetSettings"), TEXT("TargetedRHIs"), TargetedShaderFormats, GEngineIni);
		if (TargetedShaderFormats.Contains(TEXT("SF_VULKAN_SM6")))
		{
			return TEXT("SF_VULKAN_SM6");
		}
		if (TargetedShaderFormats.Contains(TEXT("SF_VULKAN_SM5")))
		{
			return TEXT("SF_VULKAN_SM5");
		}
	}

	// Default to SM6
	return TEXT("SF_VULKAN_SM6");
}

#define CPUINFO_TOKENS                            \
	_XTAG( Hardware, "Hardware" ),                \
	_XTAG( Revision, "Revision" ),                \
	_XTAG( Model, "Model" ),                      \
	_XTAG( VendorID, "vendor_id" ),               \
	_XTAG( ModelName, "model name" ),             \
	_XTAG( Processor, "Processor" ),              \
	_XTAG( CPUimplementer, "CPU implementer" ),   \
	_XTAG( CPUarchitecture, "CPU architecture" ), \
	_XTAG( CPUvariant, "CPU variant" ),           \
	_XTAG( CPUpart, "CPU part" ),                 \
	_XTAG( CPUrevision, "CPU revision" ),         \
	_XTAG( CPUfamily, "cpu family"),              \
	_XTAG( CPUmodel, "model"),                    \
	_XTAG( CPUstepping, "stepping"),              \
	_XTAG( CPUmicrocode, "microcode"),

class FPlatformProcCPUInfo
{
public:

	// True if we were able to parse /proc/cpuinfo and find (at least) CPU Vendor
	static bool IsValid()          { return (GIsValid == 1); }

	static FString GetCPUBrand()			{ ParseCPUInfoFile(); return GCPUBrand; }
	static FString GetCPUVendor()			{ ParseCPUInfoFile(); return GCPUVendor; }
	static FString GetCPUChipset()			{ ParseCPUInfoFile(); return GCPUChipset; }
	static uint32 GetCPUModelID()			{ ParseCPUInfoFile(); return GCPUModelID; }
	static uint32 GetCPUMicrocodeRevision()	{ ParseCPUInfoFile(); return GCPUMicrocodeRevision; }

private:

	static bool ParseCPUInfoFile();
	static bool ParseCPUInfoEntries(const ANSICHAR *Filename, TArray<FString>& CPUInfoValues);
	static FString GetTokenValue(const ANSICHAR *LineBuffer, const ANSICHAR *Token);

#if PLATFORM_CPU_ARM_FAMILY
	static FString GetArmCPUImplementerStr(uint32 CPUImplementer);
	static FString GetArmCPUPartStr(uint32 CPUImplementer, uint32 CPUPart);
#endif

private:

	enum
	{
#define _XTAG( _x, _y ) Token ## _x
		CPUINFO_TOKENS
#undef _XTAG
		TokensMax
	};

	static int GIsValid;

	static FString GCPUVendor;
	static FString GCPUBrand;
	static FString GCPUChipset;
	static uint32 GCPUModelID;
	static uint32 GCPUMicrocodeRevision;
};

// Make of the device we are running on eg "Marvell"
FString FPlatformProcCPUInfo::GCPUVendor;
// Model of the device we are running on eg "Marvell Armada-375 Board - ARMv7 Processor rev 1"
FString FPlatformProcCPUInfo::GCPUBrand;
// Chipset eg Cortex-A72 rev3
FString FPlatformProcCPUInfo::GCPUChipset;
uint32 FPlatformProcCPUInfo::GCPUModelID = 0;
uint32 FPlatformProcCPUInfo::GCPUMicrocodeRevision = 0;

int FPlatformProcCPUInfo::GIsValid = -1;

#if PLATFORM_CPU_ARM_FAMILY

FString FPlatformProcCPUInfo::GetArmCPUImplementerStr(uint32 CPUImplementer)
{
	switch (CPUImplementer)
	{
		case 0x41: return TEXT( "ARM" );
		case 0x42: return TEXT( "Broadcom" );
		case 0x43: return TEXT( "Cavium" );
		case 0x44: return TEXT( "DEC" );
		case 0x46: return TEXT( "Fujitsu" );
		case 0x48: return TEXT( "HiSilicon" );
		case 0x49: return TEXT( "Infineon" );
		case 0x4d: return TEXT( "Motorola" );
		case 0x4e: return TEXT( "NVIDIA" );
		case 0x50: return TEXT( "AppliedMicro" ); // Applied Micro Circuits Corporation (APM)
		case 0x51: return TEXT( "Qualcomm" );
		case 0x53: return TEXT( "Samsung" );
		case 0x54: return TEXT( "Texas Instruments" );
		case 0x56: return TEXT( "Marvell" );
		case 0x61: return TEXT( "Apple" );
		case 0x66: return TEXT( "Faraday" );
		case 0x68: return TEXT( "HXT" );
		case 0x69: return TEXT( "Intel" );
		case 0xc0: return TEXT( "Ampere Computing" );
	}

	return FString::Printf(TEXT("CPUImplementer_0x%x"), CPUImplementer);
}

FString FPlatformProcCPUInfo::GetArmCPUPartStr( uint32 CPUImplementer, uint32 CPUPart )
{
#define ARM64_CPU_MODEL(_implementer, _part) (((_part) << 8) | (_implementer))

	switch (ARM64_CPU_MODEL(CPUImplementer, CPUPart))
	{
		// ARM
		case ARM64_CPU_MODEL(0x41, 0xd02): return TEXT( "Cortex-A34" );     // ARMv8.0-A
		case ARM64_CPU_MODEL(0x41, 0xd03): return TEXT( "Cortex-A53" );     // ARMv8.0-A
		case ARM64_CPU_MODEL(0x41, 0xd04): return TEXT( "Cortex-A35" );     // ARMv8.0-A
		case ARM64_CPU_MODEL(0x41, 0xd05): return TEXT( "Cortex-A55" );     // ARMv8.2-A
		case ARM64_CPU_MODEL(0x41, 0xd06): return TEXT( "Cortex-A65" );     // ARMv8.2-A
		case ARM64_CPU_MODEL(0x41, 0xd07): return TEXT( "Cortex-A57" );     // ARMv8.0-A
		case ARM64_CPU_MODEL(0x41, 0xd08): return TEXT( "Cortex-A72" );     // ARMv8.0-A
		case ARM64_CPU_MODEL(0x41, 0xd09): return TEXT( "Cortex-A73" );     // ARMv8.0-A
		case ARM64_CPU_MODEL(0x41, 0xd0a): return TEXT( "Cortex-A75" );     // ARMv8.2-A
		case ARM64_CPU_MODEL(0x41, 0xd0b): return TEXT( "Cortex-A76" );     // ARMv8.2-A
		case ARM64_CPU_MODEL(0x41, 0xd0c): return TEXT( "Neoverse N1" );    // ARMv8.2-A
		case ARM64_CPU_MODEL(0x41, 0xd0d): return TEXT( "Cortex-A77" );     // ARMv8.2-A
		case ARM64_CPU_MODEL(0x41, 0xd0e): return TEXT( "Cortex-A76AE" );   // ARMv8.2-A
		case ARM64_CPU_MODEL(0x41, 0xd40): return TEXT( "Neoverse V1" );    // ARMv8.4+SVE(1)
		case ARM64_CPU_MODEL(0x41, 0xd41): return TEXT( "Cortex-A78" );     // ARMv8.2-A
		case ARM64_CPU_MODEL(0x41, 0xd42): return TEXT( "Cortex-A78AE" );   // ARMv8.2-A
		case ARM64_CPU_MODEL(0x41, 0xd43): return TEXT( "Cortex-A65AE" );   // ARMv8.2-A
		case ARM64_CPU_MODEL(0x41, 0xd44): return TEXT( "Cortex-X1" );      // ARMv8.2-A
		case ARM64_CPU_MODEL(0x41, 0xd46): return TEXT( "Cortex-A510" );    // ARMv9.0-A
		case ARM64_CPU_MODEL(0x41, 0xd47): return TEXT( "Cortex-A710" );    // ARMv9.0-A
		case ARM64_CPU_MODEL(0x41, 0xd48): return TEXT( "Cortex-X2" );      // ARMv9.0-A
		case ARM64_CPU_MODEL(0x41, 0xd49): return TEXT( "Neoverse N2" );    // ARMv9.0-A
		case ARM64_CPU_MODEL(0x41, 0xd4a): return TEXT( "Neoverse E1" );    // ARMv8.2-A
		case ARM64_CPU_MODEL(0x41, 0xd4b): return TEXT( "Cortex-A78C" );    // ARMv8.2-A
		// Marvell / Cavium
		case ARM64_CPU_MODEL(0x43, 0x0a0): return TEXT( "ThunderX" );       // ARMv8.0-A
		case ARM64_CPU_MODEL(0x43, 0x0a1): return TEXT( "ThunderX T88" );
		case ARM64_CPU_MODEL(0x43, 0x0a2): return TEXT( "ThunderX T81" );   // Octeon TX 81
		case ARM64_CPU_MODEL(0x43, 0x0a3): return TEXT( "ThunderX T83" );   // Octeon TX 83
		case ARM64_CPU_MODEL(0x43, 0x0af): return TEXT( "ThunderX2 T99" );  // ARMv8.1-A
		case ARM64_CPU_MODEL(0x43, 0x0b0): return TEXT( "Octeon TX2" );     // ARMv8.2
		case ARM64_CPU_MODEL(0x43, 0x0b1): return TEXT( "Octeon TX2 T98" );
		case ARM64_CPU_MODEL(0x43, 0x0b2): return TEXT( "Octeon TX2 T96" );
		case ARM64_CPU_MODEL(0x43, 0x0b3): return TEXT( "Octeon TX2 F95" );
		case ARM64_CPU_MODEL(0x43, 0x0b4): return TEXT( "Octeon TX2 F95N" );
		case ARM64_CPU_MODEL(0x43, 0x0b5): return TEXT( "Octeon TX2 F95MM" );
		case ARM64_CPU_MODEL(0x43, 0x0b8): return TEXT( "ThunderX3 T110" ); // ARMv8.3+
		// NVIDIA
		case ARM64_CPU_MODEL(0x4e, 0x000): return TEXT( "Denver" );         // ARMv8.0-A
		case ARM64_CPU_MODEL(0x4e, 0x003): return TEXT( "Denver 2" );       // ARMv8.0-A
		case ARM64_CPU_MODEL(0x4e, 0x004): return TEXT( "Carmel" );         // ARMv8.2-A
		// Applied Micro X-Gene
		case ARM64_CPU_MODEL(0x50, 0x000): return TEXT( "X-Gene" );
	}

	return FString::Printf(TEXT("CPUPart_0x%x"), CPUPart);
}

#endif // PLATFORM_CPU_ARM_FAMILY

FString FPlatformProcCPUInfo::GetTokenValue(const ANSICHAR *LineBuffer, const ANSICHAR *Token)
{
	uint32 TokenLen = FCStringAnsi::Strlen(Token);

	// Check if line starts with Token
	if (!FCStringAnsi::Strncmp(LineBuffer, Token, TokenLen))
	{
		const ANSICHAR *Separator = FCStringAnsi::Strchr(LineBuffer + TokenLen, ':');

		if (Separator)
		{
			FString Value = UTF8_TO_TCHAR(Separator + 1);

			return Value.TrimStartAndEnd();
		}
	}

	return TEXT("");
}

bool FPlatformProcCPUInfo::ParseCPUInfoEntries(const ANSICHAR *Filename, TArray<FString>& CPUInfoValues)
{
	static const ANSICHAR *TokenStrings[] =
	{
#define _XTAG( _x, _y ) _y
		CPUINFO_TOKENS
#undef _XTAG
	};

	CPUInfoValues.Empty();
	CPUInfoValues.SetNumZeroed(TokensMax);

	uint32 Count = 0;
	FILE* FsFile = fopen(Filename, "r");
	if (!FsFile)
	{
		int ErrNo = errno;
		UE_LOGF(LogCore, Warning, "Unable to fopen('%ls'): errno=%d (%ls)",
			UTF8_TO_TCHAR(Filename), ErrNo, UTF8_TO_TCHAR(strerror(ErrNo)));
	}
	else
	{
		for (;;)
		{
			// Grab line from cpuinfo file
			ANSICHAR LineBuffer[256] = { 0 };
			ANSICHAR *Line = fgets(LineBuffer, sizeof(LineBuffer), FsFile);

			if (Line == nullptr)
			{
				break;
			}

			// Go through all empty tokens checking for match with this line
			for (int i = 0; i < TokensMax; i++)
			{
				if (CPUInfoValues[i].IsEmpty())
				{
					FString Token = GetTokenValue(LineBuffer, TokenStrings[i]);

					if (!Token.IsEmpty())
					{
						CPUInfoValues[i] = Token;
						Count++;
						break;
					}
				}
			}
		}

		fclose(FsFile);
	}

	return (Count > 0);
}

bool FPlatformProcCPUInfo::ParseCPUInfoFile()
{
	if (GIsValid != -1)
	{
		return IsValid();
	}

	TArray<FString> CPUInfoValues;

	if (ParseCPUInfoEntries("/proc/cpuinfo", CPUInfoValues))
	{
		const FString& strModel          = CPUInfoValues[TokenModel];
		const FString& strHardware       = CPUInfoValues[TokenHardware];
		const FString& strModelName      = CPUInfoValues[TokenModelName];
		const FString& strProcessor      = CPUInfoValues[TokenProcessor];
		const FString& strVendorID       = CPUInfoValues[TokenVendorID];

#if PLATFORM_CPU_ARM_FAMILY
		const FString& strCPUimplementer = CPUInfoValues[TokenCPUimplementer];
		const FString& strCPUpart        = CPUInfoValues[TokenCPUpart];
		const FString& strCPUrevision    = CPUInfoValues[TokenCPUrevision];

		if (!strCPUimplementer.IsEmpty())
		{
			// Parse ARM "CPU Implementer" value. Should be 0x41, 0x43, etc.
			uint32 CPUImplementer = FCString::Strtoui64(*strCPUimplementer, nullptr, 16);
			FString CPUImplementerStr = GetArmCPUImplementerStr(CPUImplementer);

			GCPUVendor = CPUImplementerStr;

			if (!strCPUpart.IsEmpty())
			{
				// "CPU part". Should be 0xd08, etc.
				uint32 CPUPart = FCString::Strtoui64(*strCPUpart, nullptr, 16);

				GCPUChipset = GetArmCPUPartStr(CPUImplementer, CPUPart);

				// Add on "CPU revision" if not empty.
				if (!strCPUrevision.IsEmpty())
				{
					GCPUChipset += TEXT(" rev") + strCPUrevision;
				}
			}
		}
		else
#endif // PLATFORM_CPU_ARM_FAMILY
		{
			// Try using "vendor_id". Should only be on x64, but worth a try.
			GCPUVendor = strVendorID;
		}

		if (!strModel.IsEmpty())
		{
			// Use "Model" if found. Should be something like:
			//   Raspberry Pi 4 Model B Rev 1.2
			GCPUBrand = strModel;
		}
		else
		{
			// Use "Hardware" first if set. Ie: "Amlogic"
			if (!strHardware.IsEmpty())
			{
				GCPUBrand = strHardware;
			}

			if (!strModelName.IsEmpty())
			{
				// "model name" is "ARMv8 Processor rev 3 (v8l)", etc
				FString Sep = !GCPUBrand.IsEmpty() ? TEXT(" - ") : TEXT("");
				GCPUBrand += Sep + strModelName;
			}
			else if (!strProcessor.IsEmpty())
			{
				// "Processor" is something like "ARMv7 Processor rev 1 (v7l)" if set.
				FString Sep = !GCPUBrand.IsEmpty() ? TEXT(" - ") : TEXT("");
				GCPUBrand += Sep + strProcessor;
			}
		}

		if (GCPUBrand.IsEmpty())
		{
			// Fall back to generic "Cortex-A57 rev3" if above entries not found.
			GCPUBrand = GCPUChipset;
		}

		// Build a model ID from the family, model and stepping. Works for x64 CPUs.
		{
			const FString& strFamily = CPUInfoValues[TokenCPUfamily];
			const FString& strCpuModel = CPUInfoValues[TokenCPUmodel];
			const FString& strStepping = CPUInfoValues[TokenCPUstepping];
			if (!strFamily.IsEmpty() && !strCpuModel.IsEmpty() && !strStepping.IsEmpty())
			{
				uint32 FamilyID = FCString::Atoi(*strFamily);
				uint32 ModelID = FCString::Atoi(*strCpuModel);
				uint32 SteppingID = FCString::Atoi(*strStepping);
				// Use the same packing logic as FWindowsPlatformMisc::GetCPUModelID().
				GCPUModelID = (FamilyID << 12) | (ModelID << 4) | SteppingID;
			}
		}

		{
			const FString& strMicrocode = CPUInfoValues[TokenCPUmicrocode];
			if (strMicrocode.StartsWith(TEXT("0x")))
			{
				TCHAR* endPtr;
				GCPUMicrocodeRevision = FCString::Strtoi(*strMicrocode + 2, &endPtr, 16);
				if (*endPtr != 0)
				{
					GCPUMicrocodeRevision = 0;
				}
			}
		}

		// Mark success if we found a vendor at least.
		GIsValid = GCPUVendor.IsEmpty() ? 0 : 1;
	}
	else
	{
		// ParseCPUInfoEntries failed
		GIsValid = 0;
	}

	// Set to default unknown values for anything we couldn't find.
	if (GCPUBrand.IsEmpty())
	{
		GCPUBrand = TEXT("UnknownCPUBrand");
	}
	if (GCPUVendor.IsEmpty())
	{
		GCPUVendor = TEXT("UnknownCPUVendor");
	}
	if (GCPUChipset.IsEmpty())
	{
		GCPUChipset = TEXT("Unknown");
	}

	return IsValid();
}

bool FUnixPlatformMisc::HasCPUIDInstruction()
{
#if PLATFORM_HAS_CPUID
	return __get_cpuid_max(0, 0) != 0;
#else
	return false;	// Unix ARM or something more exotic
#endif // PLATFORM_HAS_CPUID
}

FString FUnixPlatformMisc::GetCPUVendor()
{
#if PLATFORM_HAS_CPUID
	static TCHAR Result[13] = TEXT("NonX86Vendor");
	static bool bHaveResult = false;

	if (!bHaveResult)
	{
		union
		{
			char Buffer[12 + 1];
			struct
			{
				int dw0;
				int dw1;
				int dw2;
			} Dw;
		} VendorResult;

		int Dummy;
		__cpuid(0, Dummy, VendorResult.Dw.dw0, VendorResult.Dw.dw2, VendorResult.Dw.dw1);

		VendorResult.Buffer[12] = 0;

		FCString::Strncpy(Result, UTF8_TO_TCHAR(VendorResult.Buffer), UE_ARRAY_COUNT(Result));

		bHaveResult = true;
	}

	return FString(Result);
#else
	return FPlatformProcCPUInfo::GetCPUVendor();
#endif // PLATFORM_HAS_CPUID
}

uint32 FUnixPlatformMisc::GetCPUInfo()
{
	static uint32 Info = 0;
	static bool bHaveResult = false;

	if (!bHaveResult)
	{
#if PLATFORM_HAS_CPUID
		int Dummy[3];
		__cpuid(1, Info, Dummy[0], Dummy[1], Dummy[2]);
#endif // PLATFORM_HAS_CPUID

		bHaveResult = true;
	}

	return Info;
}

FString FUnixPlatformMisc::GetCPUBrand()
{
#if PLATFORM_HAS_CPUID
	static FString Result = "NonX86CPUBrand";
	static bool bHaveResult = false;

	if (!bHaveResult)
	{
		// @see for more information http://msdn.microsoft.com/en-us/library/vstudio/hskdteyh(v=vs.100).aspx
		ANSICHAR BrandString[0x40] = { 0 };
		int32 CPUInfo[4] = { -1 };
		const SIZE_T CPUInfoSize = sizeof(CPUInfo);

		__cpuid(0x80000000, CPUInfo[0], CPUInfo[1], CPUInfo[2], CPUInfo[3]);
		const uint32 MaxExtIDs = CPUInfo[0];

		if (MaxExtIDs >= 0x80000004)
		{
			const uint32 FirstBrandString = 0x80000002;
			const uint32 NumBrandStrings = 3;
			for (uint32 Index = 0; Index < NumBrandStrings; ++Index)
			{
				__cpuid(FirstBrandString + Index, CPUInfo[0], CPUInfo[1], CPUInfo[2], CPUInfo[3]);
				FPlatformMemory::Memcpy(BrandString + CPUInfoSize * Index, CPUInfo, CPUInfoSize);
			}
		}

		Result = UTF8_TO_TCHAR(BrandString);
		Result.TrimStartAndEndInline();
		bHaveResult = true;
	}

	return Result;
#else
	return FPlatformProcCPUInfo::GetCPUBrand();
#endif // PLATFORM_HAS_CPUID
}

uint32 FUnixPlatformMisc::GetCPUModelID()
{
	return FPlatformProcCPUInfo::GetCPUModelID();
}

uint32 FUnixPlatformMisc::GetCPUMicrocodeRevision()
{
	return FPlatformProcCPUInfo::GetCPUMicrocodeRevision();
}

// From RHIVendorIdToString()
static const TCHAR* VendorIdToString(uint32 VendorId)
{
	switch (VendorId)
	{
		case 0x1002:  return TEXT("AMD");
		case 0x1010:  return TEXT("ImgTec");
		case 0x10DE:  return TEXT("NVIDIA");
		case 0x13B5:  return TEXT("ARM");
		case 0x14E4:  return TEXT("Broadcom");
		case 0x5143:  return TEXT("Qualcomm");
		case 0x106B:  return TEXT("Apple");
		case 0x8086:  return TEXT("Intel");
		case 0x7a05:  return TEXT("Vivante");
		case 0x1EB1:  return TEXT("VeriSilicon");
		case 0x10003: return TEXT("Kazan");
		case 0x10004: return TEXT("Codeplay");
		case 0x10005: return TEXT("Mesa");
	default:
		break;
	}

	return TEXT("UnknownVendor");
}

static FString ReadGPUFile(const char *Filename, const char *Token = nullptr)
{
	FString Value;
	FILE* FsFile = fopen(Filename, "r");

	if (FsFile)
	{
		uint32 TokenLen = Token ? FCStringAnsi::Strlen(Token) : 0;

		for (;;)
		{
			ANSICHAR LineBuffer[512];
			ANSICHAR *Line = fgets(LineBuffer, sizeof(LineBuffer), FsFile);

			if (!Line)
			{
				break;
			}

			if (!Token)
			{
				// Skip over the 0x prefix
				if (Line[0] == '0' && Line[1] == 'x')
				{
					Line += 2;
				}
				Value = UTF8_TO_TCHAR(Line);
				break;
			}
			if (!FCStringAnsi::Strncmp(Line, Token, TokenLen))
			{
				Value = UTF8_TO_TCHAR(Line) + TokenLen;
				break;
			}
		}
		Value.TrimStartAndEndInline();

		fclose(FsFile);
	}

	return Value;
}

static FString GetVendorDevicePCIIdStr()
{
	FString Vendor0 = ReadGPUFile("/sys/class/drm/card0/device/vendor");
	FString Device0 = ReadGPUFile("/sys/class/drm/card0/device/device");
	FString SubsystemVendor0 = ReadGPUFile("/sys/class/drm/card0/device/subsystem_vendor");
	FString SubsystemDevice0 = ReadGPUFile("/sys/class/drm/card0/device/subsystem_device");
	uint32 VendorId = strtoul(TCHAR_TO_UTF8(*Vendor0), nullptr, 16);
	const TCHAR* VendorStr = VendorIdToString(VendorId);

	FString ProcGPUBrandStr = VendorStr;
	if (!Device0.IsEmpty())
	{
		ProcGPUBrandStr += FString::Printf(TEXT(" PCI-id: %s-%s"), *Vendor0, *Device0);
	}
	if (!SubsystemVendor0.IsEmpty() || !SubsystemDevice0.IsEmpty())
	{
		ProcGPUBrandStr += FString::Printf(TEXT(" (%s-%s)"), *SubsystemVendor0, *SubsystemDevice0);
	}

	return ProcGPUBrandStr;
}

// Return primary GPU brand string
//  This should trying to report out what GPU is in use: Nvidia/Type/Driver version.
FString FUnixPlatformMisc::GetPrimaryGPUBrand()
{
	static FString ProcGPUBrandStr;

	if (ProcGPUBrandStr.IsEmpty())
	{
		// https://download.nvidia.com/XFree86/Linux-x86_64/440.31/README/procinterface.html
		FString NVIDIAVersion = ReadGPUFile("/proc/driver/nvidia/version", "NVRM version:");
		if (!NVIDIAVersion.IsEmpty())
		{
			// Returns something ~ like "NVIDIA GeForce RTX 3080 Ti (NVIDIA UNIX x86_64 Kernel Module  470.86  Tue Oct 26 21:55:45 UTC 2021)"
			FString GPUModel;
			glob_t GlobResults;

			// Try to read GPU Model information from:
			//   /proc/driver/nvidia/gpus/domain:bus:device.function/information
			if (glob("/proc/driver/nvidia/gpus/**/information", GLOB_NOSORT, nullptr, &GlobResults) == 0)
			{
				for (int32 Idx = 0; Idx < GlobResults.gl_pathc; Idx++)
				{
					GPUModel = ReadGPUFile(GlobResults.gl_pathv[Idx], "Model:");
					if (!GPUModel.IsEmpty())
					{
						break;
					}
				}

				globfree(&GlobResults);
			}

			// It looks like sometimes with v495 and v510 drivers we're getting this:
			// $ cat /proc/driver/nvidia/gpus/**/information
			//   Model: Unknown
			//   ...
			// If this is the case, let's try to grab the PCI-Id info.
			if (GPUModel.IsEmpty() || GPUModel == TEXT("Unknown"))
			{
				GPUModel = GetVendorDevicePCIIdStr();
			}

			ProcGPUBrandStr = FString::Printf(TEXT("%s (%s)"), *GPUModel, *NVIDIAVersion);
		}
		else
		{
			ProcGPUBrandStr = GetVendorDevicePCIIdStr();
		}
	}

	return ProcGPUBrandStr;
}

#if !UE_BUILD_SHIPPING
bool FUnixPlatformMisc::IsDebuggerPresent()
{
	extern CORE_API bool GIgnoreDebugger;
	if (GIgnoreDebugger)
	{
		return false;
	}

	// If a process is tracing this one then TracerPid in /proc/self/status will
	// be the id of the tracing process. Use SignalHandler safe functions

	int StatusFile = -1;
	bool bDebugging = false;
	UE_AUTORTFM_OPEN
	{
		StatusFile = open("/proc/self/status", O_RDONLY);

		if (StatusFile >= 0)
		{
			char Buffer[256];
			ssize_t Length = read(StatusFile, Buffer, sizeof(Buffer));

			const char* TracerString = "TracerPid:\t";
			const ssize_t LenTracerString = strlen(TracerString);
			int i = 0;

			while((Length - i) > LenTracerString)
			{
				// TracerPid is found
				if (strncmp(&Buffer[i], TracerString, LenTracerString) == 0)
				{
					// 0 if no process is tracing.
					bDebugging = Buffer[i + LenTracerString] != '0';
					break;
				}

				++i;
			}

			close(StatusFile);
		}
	};
	return bDebugging;
}
#endif // !UE_BUILD_SHIPPING

bool FUnixPlatformMisc::HasBeenStartedRemotely()
{
	static bool bHaveAnswer = false;
	static bool bResult = false;

	if (!bHaveAnswer)
	{
		const char * VarValue = secure_getenv("SSH_CONNECTION");
		bResult = (VarValue && strlen(VarValue) != 0);
		bHaveAnswer = true;
	}

	return bResult;
}

FString FUnixPlatformMisc::GetOperatingSystemId()
{
	static bool bHasCachedResult = false;
	static FString CachedResult;

	if (!bHasCachedResult)
	{
		int OsGuidFile = open("/etc/machine-id", O_RDONLY);
		if (OsGuidFile != -1)
		{
			char Buffer[PlatformMiscLimits::MaxOsGuidLength + 1] = {0};
			ssize_t ReadBytes = read(OsGuidFile, Buffer, sizeof(Buffer) - 1);

			if (ReadBytes > 0)
			{
				CachedResult = UTF8_TO_TCHAR(Buffer);
			}

			close(OsGuidFile);
		}

		// old POSIX gethostid() is not useful. It is impossible to have globally unique 32-bit GUIDs and most
		// systems don't try hard implementing it these days (glibc will return a permuted IP address, often 127.0.0.1)
		// Due to that, we just ignore that call and consider lack of systemd's /etc/machine-id a failure to obtain the host id.

		bHasCachedResult = true;	// even if we failed to read the real one
	}

	return CachedResult;
}

bool FUnixPlatformMisc::GetDiskTotalAndFreeSpace(const FString& InPath, uint64& TotalNumberOfBytes, uint64& NumberOfFreeBytes)
{
	struct statfs FSStat = { 0 };
	FTCHARToUTF8 Converter(*InPath);
	int Err = statfs((ANSICHAR*)Converter.Get(), &FSStat);
	if (Err == 0)
	{
		TotalNumberOfBytes = FSStat.f_blocks * FSStat.f_bsize;
		NumberOfFreeBytes = FSStat.f_bavail * FSStat.f_bsize;
	}
	else
	{
		int ErrNo = errno;
		UE_LOGF(LogCore, Warning, "Unable to statfs('%ls'): errno=%d (%ls)", *InPath, ErrNo, UTF8_TO_TCHAR(strerror(ErrNo)));
	}
	return (Err == 0);
}


TArray<uint8> FUnixPlatformMisc::GetMacAddress()
{
	struct ifaddrs *ifap, *ifaptr;
	TArray<uint8> Result;

	if (getifaddrs(&ifap) == 0)
	{
		for (ifaptr = ifap; ifaptr != nullptr; ifaptr = (ifaptr)->ifa_next)
		{
			struct ifreq ifr;

			strncpy(ifr.ifr_name, ifaptr->ifa_name, IFNAMSIZ-1);

			int Socket = socket(AF_UNIX, SOCK_DGRAM, 0);
			if (Socket == -1)
			{
				continue;
			}

			if (ioctl(Socket, SIOCGIFHWADDR, &ifr) == -1)
			{
				close(Socket);
				continue;
			}

			close(Socket);

			if (ifr.ifr_hwaddr.sa_family != ARPHRD_ETHER)
			{
				continue;
			}

			const uint8 *MAC = (uint8 *) ifr.ifr_hwaddr.sa_data;

			for (int32 i=0; i < 6; i++)
			{
				Result.Add(MAC[i]);
			}

			break;
		}

		freeifaddrs(ifap);
	}

	return Result;
}


static int64 LastBatteryCheck = 0;
static bool bIsOnBattery = false;

bool FUnixPlatformMisc::IsRunningOnBattery()
{
	char Scratch[8];
	FDateTime Time = FDateTime::Now();
	int64 Seconds = Time.ToUnixTimestamp();

	// don't poll the OS for battery state on every tick. Just do it once every 10 seconds.
	if (LastBatteryCheck != 0 && (Seconds - LastBatteryCheck) < 10)
	{
		return bIsOnBattery;
	}

	LastBatteryCheck = Seconds;
	bIsOnBattery = false;

	// [RCL] 2015-09-30 FIXME: find a more robust way?
	const int kHardCodedNumBatteries = 10;
	for (int IdxBattery = 0; IdxBattery < kHardCodedNumBatteries; ++IdxBattery)
	{
		char Filename[128];
		sprintf(Filename, "/sys/class/power_supply/ADP%d/online", IdxBattery);

		int State = open(Filename, O_RDONLY);
		if (State != -1)
		{
			// found ACAD device. check its state.
			ssize_t ReadBytes = read(State, Scratch, 1);
			close(State);

			if (ReadBytes > 0)
			{
				bIsOnBattery = (Scratch[0] == '0');
			}

			break;	// quit checking after we found at least one
		}
	}

	// lack of ADP most likely means that we're not on laptop at all

	return bIsOnBattery;
}

void FUnixPlatformMisc::CreateGuid(FGuid& Result)
{
	Result = FUnixPlatformGuid::CreateGuid();
}

CORE_API TFunction<void()> UngrabAllInputCallback;

void FUnixPlatformMisc::UngrabAllInput()
{
	if(UngrabAllInputCallback)
	{
		UngrabAllInputCallback();
	}
}

FString FUnixPlatformMisc::GetLoginId()
{
	return FString::Printf(TEXT("%s-%08x"), *GetOperatingSystemId(), static_cast<uint32>(geteuid()));
}

IPlatformChunkInstall* FUnixPlatformMisc::GetPlatformChunkInstall()
{
	static IPlatformChunkInstall* ChunkInstall = nullptr;
	static bool bIniChecked = false;
	if (!ChunkInstall || !bIniChecked)
	{
		IPlatformChunkInstallModule* PlatformChunkInstallModule = nullptr;
		if (!GEngineIni.IsEmpty())
		{
			FString InstallModule;
			GConfig->GetString(TEXT("StreamingInstall"), TEXT("DefaultProviderName"), InstallModule, GEngineIni);
			FModuleStatus Status;
			if (FModuleManager::Get().QueryModule(*InstallModule, Status))
			{
				PlatformChunkInstallModule = FModuleManager::LoadModulePtr<IPlatformChunkInstallModule>(*InstallModule);
				if (PlatformChunkInstallModule != nullptr)
				{
					// Attempt to grab the platform installer
					ChunkInstall = PlatformChunkInstallModule->GetPlatformChunkInstall();
				}
			}
			bIniChecked = true;
		}

		if (PlatformChunkInstallModule == nullptr)
		{
			// Placeholder instance
			ChunkInstall = FGenericPlatformMisc::GetPlatformChunkInstall();
		}
	}

	return ChunkInstall;
}

bool FUnixPlatformMisc::SetStoredValues(const FString& InStoreId, const FString& InSectionName, const TMap<FString, FString>& InKeyValues)
{
	check(!InStoreId.IsEmpty());
	check(!InSectionName.IsEmpty());

	const FString ConfigPath = FString(FPlatformProcess::ApplicationSettingsDir()) / InStoreId / FString("KeyValueStore.ini");

	FConfigFile ConfigFile;
	ConfigFile.Read(ConfigPath);

	for (auto const& InKeyValue : InKeyValues)
	{
		check(!InKeyValue.Key.IsEmpty());

		ConfigFile.SetString(*InSectionName, *InKeyValue.Key, *InKeyValue.Value);
	}

	ConfigFile.Dirty = true;
	return ConfigFile.Write(ConfigPath);
}

int32 FUnixPlatformMisc::NumberOfWorkerThreadsToSpawn()
{
	static int32 MaxServerWorkerThreads = 4;

	int32 MaxWorkerThreads = INT32_MAX;

	int32 NumberOfCores = FPlatformMisc::NumberOfCores();
	int32 NumberOfCoresIncludingHyperthreads = FPlatformMisc::NumberOfCoresIncludingHyperthreads();
	int32 NumberOfThreads = 0;

	if (NumberOfCoresIncludingHyperthreads > NumberOfCores)
	{
		NumberOfThreads = NumberOfCoresIncludingHyperthreads - 2;
	}
	else
	{
		NumberOfThreads = NumberOfCores - 1;
	}

	int32 MaxWorkerThreadsWanted = IsRunningDedicatedServer() ? MaxServerWorkerThreads : MaxWorkerThreads;
	// need to spawn at least one worker thread (see FTaskGraphImplementation)
	return FMath::Max(FMath::Min(NumberOfThreads, MaxWorkerThreadsWanted), 2);
}

namespace
{
	#if PLATFORM_CPU_X86_FAMILY
		#define ARCHITECTURE_NUMBER AUDIT_ARCH_X86_64
	#elif PLATFORM_CPU_ARM_FAMILY

		// super hacky but our bundled sysroot does not have these defined. Set them to allow us to use these
		// on newer kernels. Likely this will break if you try to use an older kernel for arm64
		#if !defined(EM_AARCH64)
			#define EM_AARCH64	183	/* ARM 64 bit */
		#endif
		#if !defined(AUDIT_ARCH_AARCH64)
			#define AUDIT_ARCH_AARCH64	(EM_AARCH64|__AUDIT_ARCH_64BIT|__AUDIT_ARCH_LE)
		#endif

		#define ARCHITECTURE_NUMBER AUDIT_ARCH_AARCH64
	#else
		#error Unknown Architecture
	#endif // PLATFORM_CPU_ARM_FAMILY

	class SyscallFilter
	{
	public:
		SyscallFilter()
		{
			// Validate the architecture
			SyscallFilters.Add(BPF_STMT(BPF_LD + BPF_W + BPF_ABS, (offsetof(struct seccomp_data, arch))));
			SyscallFilters.Add(BPF_JUMP(BPF_JMP + BPF_JEQ + BPF_K, ARCHITECTURE_NUMBER, 1, 0));
			SyscallFilters.Add(BPF_STMT(BPF_RET + BPF_K, SECCOMP_RET_KILL));

			// Load the syscall number
			SyscallFilters.Add(BPF_STMT(BPF_LD + BPF_W + BPF_ABS, (offsetof(struct seccomp_data, nr))));
		}

		void AddAllowSyscall(uint32 SyscallNumber)
		{
			// from a syscall number, we will allow it and cut out before getting to the trap/kill further down
			SyscallFilters.Add(BPF_JUMP(BPF_JMP + BPF_JEQ + BPF_K, SyscallNumber, 0, 1));
			SyscallFilters.Add(BPF_STMT(BPF_RET + BPF_K, SECCOMP_RET_ALLOW));
		}

		void AddSyscallRule(uint32 SyscallNumber, uint32 SyscallArgNumber, uint32 Value, bool bAllow, bool bBitCheck)
		{
			// if we match this syscall number, check if this syscall arg  matches or *does* not match the Value passed in based on bAllow
			SyscallFilters.Add(BPF_JUMP(BPF_JMP + BPF_JEQ + BPF_K, SyscallNumber, 0, 4));
			SyscallFilters.Add(BPF_STMT(BPF_LD  + BPF_W + BPF_ABS, static_cast<uint32>(offsetof(struct seccomp_data, args[SyscallArgNumber]))));

			// if we only want to check if the bit is enabled
			if (bBitCheck)
			{
				SyscallFilters.Add(BPF_JUMP(BPF_JMP + BPF_JSET + BPF_K, Value, bAllow, !bAllow));
			}
			else
			{
				SyscallFilters.Add(BPF_JUMP(BPF_JMP + BPF_JEQ + BPF_K, Value, bAllow, !bAllow));
			}

			SyscallFilters.Add(BPF_STMT(BPF_RET + BPF_K, SECCOMP_RET_TRAP));
			SyscallFilters.Add(BPF_STMT(BPF_RET + BPF_K, SECCOMP_RET_ALLOW));
		}

		bool FinalizeSyscallFilter()
		{
			// our final filter, this will be hit if now allow or other filter cuts us earlier
			SyscallFilters.Add(BPF_STMT(BPF_RET + BPF_K, SECCOMP_RET_TRAP));

			if (SyscallFilters.Num() > TNumericLimits<unsigned short>::Max())
			{
				UE_LOGF(LogHAL, Warning, "To many filters have been added, max is %i", TNumericLimits<unsigned short>::Max());

				return false;
			}

			struct sock_fprog SysFiltersSetup = {
				.len = static_cast<unsigned short>(SyscallFilters.Num()),
				.filter = SyscallFilters.GetData()
			};

			if (prctl(PR_SET_NO_NEW_PRIVS, 1, 0, 0, 0))
			{
				int ErrNo = errno;
				UE_LOGF(LogHAL, Warning, "prctl failed to set PR_SET_NO_NEW_PRIVS '%ls'", ANSI_TO_TCHAR(strerror(ErrNo)));

				return false;
			}

			if (prctl(PR_SET_SECCOMP, SECCOMP_MODE_FILTER, &SysFiltersSetup))
			{
				int ErrNo = errno;
				UE_LOGF(LogHAL, Warning, "prctl failed to set SECCOMP_MODE_FILTER '%ls'", ANSI_TO_TCHAR(strerror(ErrNo)));

				return false;
			}

			return true;
		}

	private:
		TArray<struct sock_filter> SyscallFilters;
	};
} // anonymous namespace

bool FUnixPlatformMisc::SetupSyscallFilters()
{
	// only allow to be setup once. filters add overhead to each syscall
	// so setting up the same of over and over again could cause much worse perf
	static bool bHasBeenSetup = false;

	if (bHasBeenSetup)
	{
		UE_LOGF(LogHAL, Warning, "Syscall Filters have already been setup");

		return false;
	}

	FString SyscallFilterPath;
	if (!FParse::Value(FCommandLine::Get(), TEXT("allowsyscallfilterfile="), SyscallFilterPath) || SyscallFilterPath.Len() <= 0)
	{
		return false;
	}

	UE_LOGF(LogHAL, Display, "Setting up System call filters.");

	SyscallFilter SyscallFilters;

	TArray<FString> Syscalls;
	if (FFileHelper::LoadFileToStringArray(Syscalls, *SyscallFilterPath))
	{
		for (const FString& SyscallRule : Syscalls)
		{
			// there are only two cases here:
			// 1) <syscall> <syscall_arg_num_zero_indexed> <value> <allow> <only_check_bit_set>
			// 2) <syscall>
			//
			// For 1) we will check if there any spaces in the line, if so we assume it will have 4 args. If its does we fail to add this rule
			// For 2) we just assume this is a syscall name and will look it up in the table and add an allow rule
			//
			// check if the SyscallRule had extra arguments or was just the name
			// if theres a space in the rule, assume arguments
			//
			// example for 1) int mprotect(void *addr, size_t len, int prot);
			// mprotect 2 4 0 1
			//
			// meaning for mprotect, its 3rd argument, if its value & 4 (PROT_EXEC) == 1 then we will get a SIGSYS raise from the kernel
			//
			// if reversed for allow
			// mprotect 2 4 1 1
			//
			// this would mean we would *only* allow mprotect if the 3rd argument had PROT_EXEC bit set
			int32 Index = 0;
			if (SyscallRule.FindChar(TEXT(' '), Index))
			{
				TArray<FString> ArgList;
				SyscallRule.ParseIntoArray(ArgList, TEXT(" "));

				if (ArgList.Num() != 5)
				{
					UE_LOGF(LogHAL, Warning, "Failed to parse syscall rule '%ls', invalid number of arguments", *SyscallRule);
				}
				else
				{
					int SyscallNumber = UnixPlatformLookupSyscallNumberFromName(ArgList[0]);

					if (SyscallNumber >= 0)
					{
						int SyscallArgNumber = FCString::Atoi(*ArgList[1]);
						int Value    = FCString::Atoi(*ArgList[2]);
						int Allow    = !!FCString::Atoi(*ArgList[3]);
						int BitCheck = !!FCString::Atoi(*ArgList[4]);

						SyscallFilters.AddSyscallRule(SyscallNumber, SyscallArgNumber, Value, Allow, BitCheck);

						UE_LOGF(LogHAL, Verbose, "Adding syscall rule '%ls'('%i') arg number %i value %i allow %i check bit only %i", *ArgList[0], SyscallNumber, SyscallArgNumber, Value, Allow, BitCheck);
					}
					else
					{
						UE_LOGF(LogHAL, Warning, "Failed to find the syscall number in the look-up table for '%ls'", *SyscallRule);
					}
				}
			}
			else
			{
				int SyscallNumber = UnixPlatformLookupSyscallNumberFromName(SyscallRule);
				if (SyscallNumber >= 0)
				{
					SyscallFilters.AddAllowSyscall(SyscallNumber);

					UE_LOGF(LogHAL, Verbose, "Adding syscall '%ls' mapped to syscall number '%i' to filter", *SyscallRule, SyscallNumber);
				}
				else
				{
					UE_LOGF(LogHAL, Warning, "Failed to find the syscall number in the look-up table for '%ls'", *SyscallRule);
				}

			}
		}
	}

	// if we have the -allowsyscallfilterfile command line *always* setup syscall filters. Even if there is no file, or it doesnt exists
	// this will mean *all* syscalls are disable and you will get a SIGSYS on a syscall
	SyscallFilters.FinalizeSyscallFilter();

	bHasBeenSetup = true;

	return true;
}

#if ENABLE_PGO_PROFILE
// presence of this symbol prevents automatic PGI initialization
int CORE_API __llvm_profile_runtime = 0;

namespace UnixPlatformMisc
{
	bool GPGICollectionUnderway = false;
}

extern "C"
{
	void __llvm_profile_initialize_file(void);
	int __llvm_profile_write_file(void);
	void __llvm_profile_reset_counters(void);
};

bool FUnixPlatformMisc::StartNewPGOCollection(const FString& AbsoluteFileName)
{ 
	const TCHAR* ProfFileEnvVar = TEXT("LLVM_PROFILE_FILE");
	if (FPlatformMisc::IsPGIActive())
	{
		UE_LOGF(LogHAL, Error, "Profiling data collection is already under way! (file being written is '%ls'", *FPlatformMisc::GetEnvironmentVariable(ProfFileEnvVar));
		return false;
	}

	UnixPlatformMisc::GPGICollectionUnderway = true;
	FPlatformMisc::SetEnvironmentVar(ProfFileEnvVar, *AbsoluteFileName);
	UE_LOGF(LogHAL, Log, "Starting PGI data collection to file '%ls'", *AbsoluteFileName);
	__llvm_profile_reset_counters();
	__llvm_profile_initialize_file();
	return true;
};

bool FUnixPlatformMisc::IsPGIActive()
{
	return UnixPlatformMisc::GPGICollectionUnderway;
}


bool FUnixPlatformMisc::StopPGOCollectionAndCloseFile()
{
	if (!FPlatformMisc::IsPGIActive())
	{
		UE_LOGF(LogHAL, Warning, "Cannot stop PGO data collection, it was not started.");
		return false;
	}

	UE_LOGF(LogHAL, Log, "Stopping PGO data collection.");

	if (__llvm_profile_write_file() != 0)
	{
		UE_LOGF(LogHAL, Error, "Error writing out PGO file.");
	}
	UnixPlatformMisc::GPGICollectionUnderway = false;
	fflush(stdout);

	// write out a fake file to make sure the previous file is no longer kept open by the LLVM machinery
	FString DummyProfileFileName = "/tmp/";
	DummyProfileFileName += FGuid::NewGuid().ToString();
	FPlatformMisc::SetEnvironmentVar(TEXT("LLVM_PROFILE_FILE"), *DummyProfileFileName);
	__llvm_profile_reset_counters();
	__llvm_profile_initialize_file();
	__llvm_profile_write_file();

	return true;
}
#endif
