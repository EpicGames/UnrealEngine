// Copyright Epic Games, Inc. All Rights Reserved.

#include "Unix/UnixPlatformOutputDevices.h"

#include "CoreGlobals.h"
#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Build.h"
#include "Misc/CommandLine.h"
#include "Misc/OutputDeviceConsole.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Misc/Parse.h"
#include "Misc/Paths.h"
#include "Unix/UnixErrorOutputDevice.h"

class FOutputDeviceError;

FString FUnixOutputDevices::GetAbsoluteLogFilename()
{
	// FIXME: this function should not exist once FGenericPlatformOutputDevices::GetAbsoluteLogFilename() returns absolute filename (see UE-25650)
	return FPaths::ConvertRelativePathToFull(FGenericPlatformOutputDevices::GetAbsoluteLogFilename());
}

class FOutputDevice* FUnixOutputDevices::GetEventLog()
{
	return NULL; // @TODO No event logging
}

FOutputDeviceError* FUnixOutputDevices::GetError()
{
	static FUnixErrorOutputDevice Singleton;
	return &Singleton;
}

