// Copyright Epic Games, Inc. All Rights Reserved.

#include "Unix/UnixPlatformProcess.h"
#include "Unix/UnixPlatformCrashContext.h"
#include "Unix/UnixPlatformRealTimeSignals.h"
#include "Unix/UnixForkPageProtector.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "Containers/StringConv.h"
#include "Logging/LogMacros.h"
#include "HAL/FileManager.h"
#include "HAL/IConsoleManager.h"
#include "Misc/Parse.h"
#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "Misc/CommandLine.h"
#include "Misc/Paths.h"
#include "Unix/UnixPlatformRunnableThread.h"
#include "Misc/EngineVersion.h"
#include <dirent.h>
#include <signal.h>
#include <spawn.h>
#include <sys/wait.h>
#include <sys/resource.h>
#include <sys/ioctl.h>
#include <sys/file.h>
#include <asm/ioctls.h>
#include <sys/prctl.h>
#include "Unix/UnixPlatformOutputDevices.h"
#include "Unix/UnixPlatformTLS.h"
#include "Containers/CircularQueue.h"
#include "Misc/FileHelper.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Misc/CoreDelegates.h"
#include "Misc/App.h"
#include "Misc/Fork.h"
#include "HAL/PlatformTime.h"
#include "Misc/DateTime.h"

DECLARE_LOG_CATEGORY_CLASS(LogFork, Log, All);
DECLARE_LOG_CATEGORY_CLASS(LogUtilization, Verbose, All);

namespace PlatformProcessLimits
{
	enum
	{
		MaxUserHomeDirLength = UNIX_MAX_PATH + 1
	};
};

namespace UnixPlatformProcess
{
	static float GParentSleepDurationInSec = 10.0f;
	static FAutoConsoleVariableRef CVarForkParentSleepDurationInSec(TEXT("fork.ParentSleepDurationInSec"), 
		GParentSleepDurationInSec,
		TEXT("The time in seconds the parent process will sleep when it has no more signals to process."), 
		ECVF_Default);

	static bool GLogMemoryStatsWhenForking = true;
	static FAutoConsoleVariableRef CVarLogMemoryStatsWhenForking(TEXT("fork.LogMemoryStatsWhenForking"),
		GLogMemoryStatsWhenForking,
		TEXT("When true the parent process will log memory stats before every fork."),
		ECVF_Default);

	static float GParentSpawnDelay = 0.125f;
	static TAutoConsoleVariable<float> CVarForkParentSpawnDelay(
		TEXT("server.WaitAndForkDelay"),
		GParentSpawnDelay,
		TEXT("Controls the minimum time between processing queued fork requests"),
		ECVF_Default);

	static int GParentForkBatchSize = 1;
	static TAutoConsoleVariable<int> CVarForkBatchSize(
		TEXT("server.ForkBatchSize"),
		GParentForkBatchSize,
		TEXT("Controls the number of queued forks that will be processed when a signal is received"),
		ECVF_Default);


	FString GetReadableTime(double InDeltaSeconds)
	{
		FDateTime Now = FDateTime::UtcNow();
		const FTimespan Delta = FTimespan::FromSeconds(InDeltaSeconds);
		Now -= Delta;
		return Now.ToString(TEXT("%H.%M.%S:%s"));
	}
}

#if IS_MONOLITHIC
__thread uint32 FUnixTLS::ThreadIdTLS = 0;
#else
uint32 FUnixTLS::ThreadIdTLSKey = FUnixTLS::AllocTlsSlot();
#endif

#if UE_CHECK_LARGE_ALLOCATIONS
static TAutoConsoleVariable<int32> CVarEnableLargeAllocationChecksAfterFork(
	TEXT("memory.EnableLargeAllocationChecksAfterFork"),
	false,
	TEXT("After forking, Turn on ensure which checks no single allocation is greater than 'LargeAllocationThreshold'"),
	ECVF_Default);
#endif

void* FUnixPlatformProcess::GetDllHandle( const TCHAR* Filename )
{
	check( Filename );
	FString AbsolutePath = FPaths::ConvertRelativePathToFull(Filename);

	// first of all open the lib in LOCAL mode (we will eventually move to GLOBAL if required)
	int DlOpenMode = RTLD_LAZY;
	void *Handle = dlopen( TCHAR_TO_UTF8(*AbsolutePath), DlOpenMode | RTLD_LOCAL );
	if (Handle)
	{
		bool UpgradeToGlobal = false;
		// check for the "ue4_module_options" symbol
		const char **ue4_module_options = (const char **)dlsym(Handle, "ue4_module_options");
		if (ue4_module_options)
		{
			// split by ','
			TArray<FString> Options;
			FString UE4ModuleOptions = FString(ANSI_TO_TCHAR(*ue4_module_options));
			int32 OptionsNum = UE4ModuleOptions.ParseIntoArray(Options, ANSI_TO_TCHAR(","), true);
			for(FString Option : Options)
			{
				if (Option.Equals(FString(ANSI_TO_TCHAR("linux_global_symbols")), ESearchCase::IgnoreCase))
				{
					UpgradeToGlobal = true;
				}
			}
		}
		else
		{
			// is it ia ue4 module ? if not, move it to GLOBAL
			void *IsUE4Module = dlsym(Handle, "ThisIsAnUnrealEngineModule");
			if (!IsUE4Module)
			{
				IsUE4Module = dlsym(Handle, "InitializeModule");
			}

			if (!IsUE4Module)
			{
				UpgradeToGlobal = true;
			}
		}

		if (UpgradeToGlobal)
		{
			dlclose( Handle );
			Handle = dlopen( TCHAR_TO_UTF8(*AbsolutePath), DlOpenMode | RTLD_GLOBAL );
		}
	} 
	else if (!FString(Filename).Contains(TEXT("/")))
	{
		// if not found and the filename did not contain a path we search for it in the global path
		Handle = dlopen( TCHAR_TO_UTF8(Filename), DlOpenMode | RTLD_GLOBAL );
	}

	if (!Handle)
	{
		UE_LOGF(LogCore, Warning, "dlopen failed: %ls", UTF8_TO_TCHAR(dlerror()) );
	}

	return Handle;
}

void FUnixPlatformProcess::FreeDllHandle( void* DllHandle )
{
	check( DllHandle );
	dlclose( DllHandle );
}

void* FUnixPlatformProcess::GetDllExport( void* DllHandle, const TCHAR* ProcName )
{
	check(DllHandle);
	check(ProcName);
	return dlsym( DllHandle, TCHAR_TO_ANSI(ProcName) );
}

const TCHAR* FUnixPlatformProcess::GetModulePrefix()
{
	return TEXT("lib");
}

const TCHAR* FUnixPlatformProcess::GetModuleExtension()
{
	return TEXT("so");
}

namespace PlatformProcessLimits
{
	enum
	{
		MaxComputerName	= 128,
		MaxBaseDirLength= UNIX_MAX_PATH + 1,
		MaxArgvParameters = 256,
		MaxUserName = LOGIN_NAME_MAX
	};
};

const TCHAR* FUnixPlatformProcess::ComputerName()
{
	static bool bHaveResult = false;
	static TCHAR CachedResult[ PlatformProcessLimits::MaxComputerName ];
	if (!bHaveResult)
	{
		struct utsname name;
		const char * SysName = name.nodename;
		if(uname(&name))
		{
			SysName = "Unix Computer";
		}

		FCString::Strncpy(CachedResult, UTF8_TO_TCHAR(SysName), UE_ARRAY_COUNT(CachedResult));
		CachedResult[UE_ARRAY_COUNT(CachedResult) - 1] = 0;
		bHaveResult = true;
	}
	return CachedResult;
}

const TCHAR* FUnixPlatformProcess::UserName(bool bOnlyAlphaNumeric)
{
	static TCHAR Name[PlatformProcessLimits::MaxUserName] = { 0 };
	static TCHAR AlphaNumericName[PlatformProcessLimits::MaxUserName] = { 0 };
	static bool bHaveResult = false;
	static bool bHaveAlphaNumericResult = false;

	if ((!bHaveResult && !bOnlyAlphaNumeric) || (!bHaveAlphaNumericResult && bOnlyAlphaNumeric))
	{
		struct passwd * UserInfo = getpwuid(geteuid());
		if (nullptr != UserInfo && nullptr != UserInfo->pw_name)
		{
			FString TempName(UTF8_TO_TCHAR(UserInfo->pw_name));
			if (bOnlyAlphaNumeric)
			{
				const TCHAR *Src = *TempName;
				TCHAR * Dst = AlphaNumericName;
				for (; *Src != 0 && (Dst - AlphaNumericName) < UE_ARRAY_COUNT(AlphaNumericName) - 1; ++Src)
				{
					if (FChar::IsAlnum(*Src))
					{
						*Dst++ = *Src;
					}
				}
				*Dst++ = 0;
				bHaveAlphaNumericResult = true;
			}
			else
			{
				FCString::Strncpy(Name, *TempName, UE_ARRAY_COUNT(Name) - 1);
				bHaveResult = true;
			}
		}
		else
		{
			FCString::Sprintf(Name, TEXT("euid%d"), geteuid());
			FCString::Sprintf(AlphaNumericName, TEXT("euid%d"), geteuid());
			bHaveResult = true;
			bHaveAlphaNumericResult = true;
		}
	}

	return bOnlyAlphaNumeric ? AlphaNumericName : Name;
}

const TCHAR* FUnixPlatformProcess::UserTempDir()
{
	// Use $TMPDIR if its set otherwise fallback to /var/tmp as Windows defaults to %TEMP% which does not get cleared on reboot.
	static bool bHaveTemp = false;
	static TCHAR CachedResult[PlatformProcessLimits::MaxUserHomeDirLength] = { 0 };

	if (!bHaveTemp)
	{
		const char* TmpDirValue = secure_getenv("TMPDIR");
		if (TmpDirValue)
		{
			FCString::Strcpy(CachedResult, UTF8_TO_TCHAR(TmpDirValue));
		}
		else
		{
			FCString::Strcpy(CachedResult, TEXT("/var/tmp"));
		}

		bHaveTemp = true;
	}

	return CachedResult;
}

const TCHAR* FUnixPlatformProcess::UserDir()
{
	// The UserDir is where user visible files (such as game projects) live.
	// On Unix (just like on Mac) this corresponds to $HOME/Documents.
	// To accomodate localization requirement we use xdg-user-dir command,
	// and fall back to $HOME/Documents if setting not found.
	static TCHAR Result[UNIX_MAX_PATH] = {0};

	if (!Result[0])
	{
		FILE* FilePtr = popen("xdg-user-dir DOCUMENTS", "r");
		if (FilePtr)
		{
			char DocPath[UNIX_MAX_PATH];
			if (fgets(DocPath, UNIX_MAX_PATH, FilePtr) != nullptr)
			{
				size_t DocLen = strlen(DocPath) - 1;
				if (DocLen > 0)
				{
					DocPath[DocLen] = '\0';
					FCString::Strncpy(Result, UTF8_TO_TCHAR(DocPath), UE_ARRAY_COUNT(Result));
					FCString::StrncatTruncateDest(Result, UE_ARRAY_COUNT(Result), TEXT("/"));
				}
			}
			pclose(FilePtr);
		}

		// if xdg-user-dir did not work, use $HOME
		if (!Result[0])
		{
			FCString::Strncpy(Result, FPlatformProcess::UserHomeDir(), UE_ARRAY_COUNT(Result));
			FCString::StrncatTruncateDest(Result, UE_ARRAY_COUNT(Result), TEXT("/Documents/"));
		}
	}
	return Result;
}

const TCHAR* FUnixPlatformProcess::UserHomeDir()
{
	static bool bHaveHome = false;
	static TCHAR CachedResult[PlatformProcessLimits::MaxUserHomeDirLength] = { 0 };

	if (!bHaveHome)
	{
		bHaveHome = true;
		//  get user $HOME var first
		const char * VarValue = secure_getenv("HOME");
		if (VarValue && VarValue[0] != '\0')
		{
			FCString::Strncpy(CachedResult, UTF8_TO_TCHAR(VarValue), UE_ARRAY_COUNT(CachedResult));
		}
		else
		{
			struct passwd * UserInfo = getpwuid(geteuid());
			if (NULL != UserInfo && NULL != UserInfo->pw_dir && UserInfo->pw_dir[0] != '\0')
			{
				FCString::Strncpy(CachedResult, UTF8_TO_TCHAR(UserInfo->pw_dir), UE_ARRAY_COUNT(CachedResult));
			}
			else
			{
				FCString::Strncpy(CachedResult, FUnixPlatformProcess::UserTempDir(), UE_ARRAY_COUNT(CachedResult));
				UE_LOGF(LogInit, Warning, "Could get determine user home directory.  Using temporary directory: %ls", CachedResult);
			}
		}
	}

	return CachedResult;
}

const TCHAR* FUnixPlatformProcess::UserSettingsDir()
{
	// Like on Mac we use the same folder for UserSettingsDir and ApplicationSettingsDir
	// $HOME/.config/Epic/
	return ApplicationSettingsDir();
}

const TCHAR* FUnixPlatformProcess::ApplicationSettingsDir()
{
	// The ApplicationSettingsDir is where the engine stores settings and configuration
	// data.  On linux this corresponds to $HOME/.config/Epic
	static TCHAR Result[UNIX_MAX_PATH] = TEXT("");
	if (!Result[0])
	{
		FCString::Strncpy(Result, FPlatformProcess::UserHomeDir(), UE_ARRAY_COUNT(Result));
		FCString::StrncatTruncateDest(Result, UE_ARRAY_COUNT(Result), TEXT("/.config/Epic/"));
	}
	return Result;
}

FString FUnixPlatformProcess::GetApplicationSettingsDir(const ApplicationSettingsContext& Settings)
{
	// The ApplicationSettingsDir is where the engine stores settings and configuration
	// data.  On linux this corresponds to $HOME/.config/Epic
	TCHAR Result[UNIX_MAX_PATH] = TEXT("");
	FCString::Strncpy(Result, FPlatformProcess::UserHomeDir(), UE_ARRAY_COUNT(Result));
	if (Settings.bIsEpic)
	{
		FCString::StrncatTruncateDest(Result, UE_ARRAY_COUNT(Result), TEXT("/.config/Epic/"));
	}
	else
	{
		FCString::StrncatTruncateDest(Result, UE_ARRAY_COUNT(Result), TEXT("/.config/"));
	}
	return FString(Result);
}

bool FUnixPlatformProcess::SetProcessLimits(EProcessResource::Type Resource, uint64 Limit)
{
	rlimit NativeLimit;

	static_assert(sizeof(long) == sizeof(NativeLimit.rlim_cur), "Platform has atypical rlimit type.");

	// 32-bit platforms set limits as long
	if (sizeof(NativeLimit.rlim_cur) < sizeof(Limit))
	{
		long Limit32 = static_cast<long>(FMath::Min(Limit, static_cast<uint64>(INT_MAX)));
		NativeLimit.rlim_cur = Limit32;
		NativeLimit.rlim_max = Limit32;
	}
	else
	{
		NativeLimit.rlim_cur = Limit;
		NativeLimit.rlim_max = Limit;
	}

	int NativeResource = RLIMIT_AS;

	switch(Resource)
	{
		case EProcessResource::VirtualMemory:
			NativeResource = RLIMIT_AS;
			break;

		default:
			UE_LOGF(LogHAL, Warning, "Unknown resource type %d", Resource);
			return false;
	}

	if (setrlimit(NativeResource, &NativeLimit) != 0)
	{
		int ErrNo = errno;
		UE_LOGF(LogHAL, Warning, "setrlimit(%d, limit_cur=%ld, limit_max=%ld) failed with error %d (%ls)\n", NativeResource, NativeLimit.rlim_cur, NativeLimit.rlim_max, ErrNo, UTF8_TO_TCHAR(strerror(ErrNo)));
		return false;
	}

	return true;
}

const FString FUnixPlatformProcess::GetModulesDirectory()
{
	static FString CachedModulePath;
	if (CachedModulePath.IsEmpty())
	{
		CachedModulePath = FPaths::GetPath(FString(ExecutablePath()));
	}

	return CachedModulePath;
}

const TCHAR* FUnixPlatformProcess::ExecutablePath()
{
	static bool bHaveResult = false;
	static TCHAR CachedResult[ PlatformProcessLimits::MaxBaseDirLength ];
	if (!bHaveResult)
	{
		char SelfPath[ PlatformProcessLimits::MaxBaseDirLength ] = {0};
		if (readlink( "/proc/self/exe", SelfPath, UE_ARRAY_COUNT(SelfPath) - 1) == -1)
		{
			int ErrNo = errno;
			UE_LOGF(LogHAL, Fatal, "readlink() failed with errno = %d (%ls)", ErrNo,
				StringCast< TCHAR >(strerror(ErrNo)).Get());
			return CachedResult;
		}
		SelfPath[UE_ARRAY_COUNT(SelfPath) - 1] = 0;

		FCString::Strncpy(CachedResult, UTF8_TO_TCHAR(SelfPath), UE_ARRAY_COUNT(CachedResult));
		CachedResult[UE_ARRAY_COUNT(CachedResult) - 1] = 0;
		bHaveResult = true;
	}
	return CachedResult;
}

const TCHAR* FUnixPlatformProcess::ExecutableName(bool bRemoveExtension)
{
	static bool bHaveResult = false;
	static TCHAR CachedResult[ PlatformProcessLimits::MaxBaseDirLength ];
	if (!bHaveResult)
	{
		char SelfPath[ PlatformProcessLimits::MaxBaseDirLength ] = {0};
		if (readlink( "/proc/self/exe", SelfPath, UE_ARRAY_COUNT(SelfPath) - 1) == -1)
		{
			int ErrNo = errno;
			UE_LOGF(LogHAL, Fatal, "readlink() failed with errno = %d (%ls)", ErrNo,
				StringCast< TCHAR >(strerror(ErrNo)).Get());
			return CachedResult;
		}
		SelfPath[UE_ARRAY_COUNT(SelfPath) - 1] = 0;

		FCString::Strncpy(CachedResult, UTF8_TO_TCHAR(basename(SelfPath)), UE_ARRAY_COUNT(CachedResult));
		CachedResult[UE_ARRAY_COUNT(CachedResult) - 1] = 0;
		bHaveResult = true;
	}
	return CachedResult;
}


FString FUnixPlatformProcess::GenerateApplicationPath( const FString& AppName, EBuildConfiguration BuildConfiguration)
{
	FString PlatformName = FPlatformProcess::GetBinariesSubdirectory();
	FString ExecutablePath = FPaths::EngineDir() / FString::Printf(TEXT("Binaries/%s/%s"), *PlatformName, *AppName);
	
	if (BuildConfiguration != EBuildConfiguration::Development)
	{
		ExecutablePath += FString::Printf(TEXT("-%s-%s"), *PlatformName, LexToString(BuildConfiguration));
	}
	return ExecutablePath;
}


FString FUnixPlatformProcess::GetApplicationName( uint32 ProcessId )
{
	FString Output;

	const int32 ReadLinkSize = 1024;	
	char ReadLinkCmd[ReadLinkSize] = {0};
	FCStringAnsi::Sprintf(ReadLinkCmd, "/proc/%d/exe", ProcessId);
	
	char ProcessPath[ PlatformProcessLimits::MaxBaseDirLength ] = {0};
	int32 Ret = readlink(ReadLinkCmd, ProcessPath, UE_ARRAY_COUNT(ProcessPath) - 1);
	if (Ret != -1)
	{
		Output = UTF8_TO_TCHAR(ProcessPath);
	}
	return Output;
}

FPipeHandle::~FPipeHandle()
{
	close(PipeDesc);
}

FString FPipeHandle::Read()
{
	const int kBufferSize = 4096;
	ANSICHAR Buffer[kBufferSize];
	FString Output;

	int BytesAvailable = 0;
	if (ioctl(PipeDesc, FIONREAD, &BytesAvailable) == 0)
	{
		if (BytesAvailable > 0)
		{
			int BytesRead = read(PipeDesc, Buffer, kBufferSize - 1);
			if (BytesRead > 0)
			{
				Buffer[BytesRead] = 0;
				Output += StringCast< TCHAR >(Buffer).Get();
			}
		}
	}
	else
	{
		UE_LOGF(LogHAL, Fatal, "ioctl(..., FIONREAD, ...) failed with errno=%d (%ls)", errno, StringCast< TCHAR >(strerror(errno)).Get());
	}

	return Output;
}

bool FPipeHandle::ReadToArray(TArray<uint8> & Output)
{
	int BytesAvailable = 0;
	if (ioctl(PipeDesc, FIONREAD, &BytesAvailable) == 0)
	{
		if (BytesAvailable > 0)
		{
			Output.SetNumUninitialized(BytesAvailable);
			int BytesRead = read(PipeDesc, Output.GetData(), BytesAvailable);
			if (BytesRead > 0)
			{
				if (BytesRead < BytesAvailable)
				{
					Output.SetNum(BytesRead);
				}

				return true;
			}
			else
			{
				Output.Empty();
			}
		}
	}

	return false;
}


void FUnixPlatformProcess::ClosePipe( void* ReadPipe, void* WritePipe )
{
	if (ReadPipe)
	{
		FPipeHandle * PipeHandle = reinterpret_cast< FPipeHandle* >(ReadPipe);
		delete PipeHandle;
	}

	if (WritePipe)
	{
		FPipeHandle * PipeHandle = reinterpret_cast< FPipeHandle* >(WritePipe);
		delete PipeHandle;
	}
}

bool FUnixPlatformProcess::CreatePipe(void*& ReadPipe, void*& WritePipe, bool bWritePipeLocal)
{
	int PipeFd[2];
	if (-1 == pipe(PipeFd))
	{
		int ErrNo = errno;
		UE_LOGF(LogHAL, Warning, "pipe() failed with errno = %d (%ls)", ErrNo, 
			StringCast< TCHAR >(strerror(ErrNo)).Get());
		return false;
	}

	ReadPipe = new FPipeHandle(PipeFd[ 0 ], PipeFd[ 1 ]);
	WritePipe = new FPipeHandle(PipeFd[ 1 ], PipeFd[ 0 ]);

	return true;
}

FString FUnixPlatformProcess::ReadPipe( void* ReadPipe )
{
	if (ReadPipe)
	{
		FPipeHandle * PipeHandle = reinterpret_cast< FPipeHandle* >(ReadPipe);
		return PipeHandle->Read();
	}

	return FString();
}

bool FUnixPlatformProcess::ReadPipeToArray(void* ReadPipe, TArray<uint8> & Output)
{
	if (ReadPipe)
	{
		FPipeHandle * PipeHandle = reinterpret_cast<FPipeHandle*>(ReadPipe);
		return PipeHandle->ReadToArray(Output);
	}

	return false;
}

bool FUnixPlatformProcess::WritePipe(void* WritePipe, const FString& Message, FString* OutWritten)
{
	// if there is not a message or WritePipe is null
	int32 MessageLen = Message.Len();
	if ((MessageLen == 0) || (WritePipe == nullptr))
	{
		return false;
	}

	// Convert input to UTF8CHAR
	const TCHAR* MessagePtr = *Message;
	int32 BytesAvailable = FPlatformString::ConvertedLength<UTF8CHAR>(MessagePtr, MessageLen);
	UTF8CHAR* Buffer = new UTF8CHAR[BytesAvailable + 2];
	*FPlatformString::Convert(Buffer, BytesAvailable, MessagePtr, MessageLen) = (UTF8CHAR)'\n';

	// write to pipe
	uint32 BytesWritten = write(*(int*)WritePipe, Buffer, BytesAvailable + 1);

	// Get written message
	if (OutWritten)
	{
		*OutWritten = StringCast<TCHAR>(Buffer, BytesWritten).Get();
	}

	delete[] Buffer;
	return (BytesWritten == BytesAvailable);
}

bool FUnixPlatformProcess::WritePipe(void* WritePipe, const uint8* Data, const int32 DataLength, int32* OutDataLength)
{
	// if there is not a message or WritePipe is null
	if ((DataLength == 0) || (WritePipe == nullptr))
	{
		return false;
	}

	// write to pipe
	uint32 BytesWritten = write(*(int*)WritePipe, Data, DataLength);

	// Get written Data Length
	if (OutDataLength)
	{
		*OutDataLength = (int32)BytesWritten;
	}

	return (BytesWritten == DataLength);
}

FRunnableThread* FUnixPlatformProcess::CreateRunnableThread()
{
	return new FRunnableThreadUnix();
}

bool FUnixPlatformProcess::CanLaunchURL(const TCHAR* URL)
{
	return URL != nullptr;
}

void FUnixPlatformProcess::LaunchURL(const TCHAR* URL, const TCHAR* Parms, FString* Error)
{
	if (FCoreDelegates::ShouldLaunchUrl.IsBound() && !FCoreDelegates::ShouldLaunchUrl.Execute(URL))
	{
		if (Error)
		{
			*Error = TEXT("LaunchURL cancelled by delegate");
		}
		return;
	}

	// @todo This ignores params and error; mostly a stub
	pid_t pid = fork();
	UE_LOGF(LogHAL, Verbose, "FUnixPlatformProcess::LaunchURL: '%ls'", URL);
	if (pid == 0)
	{
		exit(execl("/usr/bin/xdg-open", "xdg-open", TCHAR_TO_UTF8(URL), (char *)0));
	}
}

/**
 * This class exists as an imperfect workaround to allow both "fire and forget" children and children about whose return code we actually care.
 * (maybe we could fork and daemonize ourselves for the first case instead?)
 */
struct FChildWaiterThread : public FRunnable
{
	/** Global table of all waiter threads */
	static TArray<FChildWaiterThread *>		ChildWaiterThreadsArray;

	/** Lock guarding the acess to child waiter threads */
	static FCriticalSection					ChildWaiterThreadsArrayGuard;

	/** Pid of child to wait for */
	int ChildPid;

	FChildWaiterThread(pid_t InChildPid)
		:	ChildPid(InChildPid)
	{
		// add ourselves to thread array
		ChildWaiterThreadsArrayGuard.Lock();
		ChildWaiterThreadsArray.Add(this);
		ChildWaiterThreadsArrayGuard.Unlock();
	}

	virtual ~FChildWaiterThread()
	{
		// remove
		ChildWaiterThreadsArrayGuard.Lock();
		ChildWaiterThreadsArray.RemoveSingle(this);
		ChildWaiterThreadsArrayGuard.Unlock();
	}

	virtual uint32 Run()
	{
		for(;;)	// infinite loop in case we get EINTR and have to repeat
		{
			siginfo_t SignalInfo;
			if (waitid(P_PID, ChildPid, &SignalInfo, WEXITED))
			{
				if (errno != EINTR)
				{
					int ErrNo = errno;
					UE_LOGF(LogHAL, Fatal, "FChildWaiterThread::Run(): waitid for pid %d failed (errno=%d, %ls)", 
							 static_cast< int32 >(ChildPid), ErrNo, UTF8_TO_TCHAR(strerror(ErrNo)));
					break;	// exit the loop if for some reason Fatal log (above) returns
				}
			}
			else
			{
				check(SignalInfo.si_pid == ChildPid);
				break;
			}
		}

		return 0;
	}

	virtual void Exit()
	{
		// unregister from the array
		delete this;
	}
};

/** See FChildWaiterThread */
TArray<FChildWaiterThread *> FChildWaiterThread::ChildWaiterThreadsArray;
/** See FChildWaiterThread */
FCriticalSection FChildWaiterThread::ChildWaiterThreadsArrayGuard;

namespace UnixPlatformProcess
{
	/**
	 * This function tries to set exec permissions on the file (if it is missing them).
	 * It exists because files copied manually from foreign filesystems (e.g. CrashReportClient) or unzipped from
	 * certain arhcive types may lack +x, yet we still want to execute them.
	 *
	 * @param AbsoluteFilename absolute filename to the file in question
	 *
	 * @return true if we should attempt to execute the file, false if it is not worth even trying
	 */	
	bool AttemptToMakeExecIfNotAlready(const FString & AbsoluteFilename)
	{
		bool bWorthTryingToExecute = true;	// be conservative and let the OS decide in most cases

		FTCHARToUTF8 AbsoluteFilenameUTF8Buffer(*AbsoluteFilename);
		const char* AbsoluteFilenameUTF8 = AbsoluteFilenameUTF8Buffer.Get();

		struct stat FilePerms;
		if (UNLIKELY(stat(AbsoluteFilenameUTF8, &FilePerms) == -1))
		{
			int ErrNo = errno;
			UE_LOGF(LogHAL, Warning, "UnixPlatformProcess::AttemptToMakeExecIfNotAlready: could not stat '%ls', errno=%d (%ls)",
				*AbsoluteFilename,
				ErrNo,
				UTF8_TO_TCHAR(strerror(ErrNo))
				);
		}
		else
		{
			// Try to make a guess if we can execute the file. We are not trying to do the exact check,
			// so if any of executable bits are set, assume it's executable
			if ((FilePerms.st_mode & (S_IXUSR | S_IXGRP | S_IXOTH)) == 0)
			{
				// if no executable bits at all, try setting permissions
				if (chmod(AbsoluteFilenameUTF8, FilePerms.st_mode | S_IXUSR) == -1)
				{
					int ErrNo = errno;
					UE_LOGF(LogHAL, Warning, "UnixPlatformProcess::AttemptToMakeExecIfNotAlready: could not chmod +x '%ls', errno=%d (%ls)",
						*AbsoluteFilename,
						ErrNo,
						UTF8_TO_TCHAR(strerror(ErrNo))
						);

					// at this point, assume that execution will fail
					bWorthTryingToExecute = false;
				}
			}
		}

		return bWorthTryingToExecute;
	}
}

FProcHandle FUnixPlatformProcess::CreateProc(const TCHAR* URL, const TCHAR* Parms, bool bLaunchDetached, bool bLaunchHidden, bool bLaunchReallyHidden, uint32* OutProcessID, int32 PriorityModifier, const TCHAR* OptionalWorkingDirectory, void* PipeWriteChild, void* PipeReadChild)
{
	// CreateProc used to only have a single "write" pipe argument, which Windows and Mac would pipe both stdout and stderr into.
	// On Unix though, only stdout was piped to it, and stderr wasn't available at all, so we'll preserve that behaviour in this overload for compatibility with existing code
	return CreateProcInternal(URL, Parms, bLaunchDetached, bLaunchHidden, bLaunchReallyHidden, OutProcessID, PriorityModifier, OptionalWorkingDirectory, PipeWriteChild, PipeReadChild, PipeWriteChild, false);
}

FProcHandle FUnixPlatformProcess::CreateProc(const TCHAR* URL, const TCHAR* Parms, bool bLaunchDetached, bool bLaunchHidden, bool bLaunchReallyHidden, uint32* OutProcessID, int32 PriorityModifier, const TCHAR* OptionalWorkingDirectory, void* PipeWriteChild, void* PipeReadChild, void* PipeStdErrChild)
{
	return CreateProcInternal(URL, Parms, bLaunchDetached, bLaunchHidden, bLaunchReallyHidden, OutProcessID, PriorityModifier, OptionalWorkingDirectory, PipeWriteChild, PipeReadChild, PipeStdErrChild, false);
}

TTuple<FProcHandle, UE::HAL::EProcessId> FUnixPlatformProcess::CreateProc(const UE::HAL::FProcessStartInfo& StartInfo)
{
	uint32 Id;
	FProcHandle Handle = CreateProcInternal(StartInfo.Uri, StartInfo.Arguments, StartInfo.bDetached, false, StartInfo.bHidden, &Id, StartInfo.PriorityModifier, StartInfo.WorkingDirectory, StartInfo.StdOut, StartInfo.StdIn, StartInfo.StdErr, StartInfo.bInheritHandles);
	return {Handle, UE::HAL::EProcessId{Id}};
}

static char* MallocedUtf8FromString(const FString& Str)
{
	FTCHARToUTF8 AnsiBuffer(*Str);
	const char* Ansi = AnsiBuffer.Get();
	size_t AnsiSize = FCStringAnsi::Strlen(Ansi) + 1;	// will work correctly with UTF-8
	check(AnsiSize);

	char* Ret = reinterpret_cast<char*>(FMemory::Malloc(AnsiSize));
	check(Ret);

	FCStringAnsi::Strncpy(Ret, Ansi, AnsiSize);	// will work correctly with UTF-8
	return Ret;
}

static bool AddCmdLineArgumentTo(char** Argv, int& Argc, const FString& CurArg)
{
	if (Argc == PlatformProcessLimits::MaxArgvParameters)
	{
		UE_LOGF(LogHAL, Warning, "FUnixPlatformProcess::CreateProc: too many (%d) commandline arguments passed, will only pass %d",
			Argc, PlatformProcessLimits::MaxArgvParameters);
		return false;
	}
	Argv[Argc] = MallocedUtf8FromString(CurArg);
	UE_LOGF(LogHAL, Verbose, "FUnixPlatformProcess::CreateProc: Argv[%d] = '%ls'", Argc, *CurArg);
	Argc++;
	return true;
}

// returns true if the token completes the current argument
static bool ParseCmdLineToken(const TCHAR* token, bool& OutIsInString, bool& OutHasArg, FString& OutCurArg, bool& OutEOL)
{
	if (*token == TEXT('\0'))
	{
		OutEOL = true;
		return OutHasArg;
	}

	if (*token == TEXT('"'))
	{
		// need to make sure this isn't a double-double quoted path
		// if we're currently in a string, then a double quote will only end a string if the next character is not a whitespace character
		if (OutIsInString)
		{
			// peek ahead to see if this looks like the start or end of a double-double quoted string
			FString temp(token + 1);
			bool StartDoubleDoubleQuote = !temp.IsEmpty() && !temp.StartsWith(TEXT(" ")) && !temp.StartsWith(TEXT("\n")) && !temp.StartsWith(TEXT("\r"));
			bool EndDoubleDoubleQuote = OutCurArg.StartsWith(TEXT("\"")) && temp.StartsWith(TEXT("\""));

			if (StartDoubleDoubleQuote || EndDoubleDoubleQuote)
			{
				// need to capture this quote into the argument
				OutCurArg += *token;
				OutHasArg = true;

				return false;
			}
		}

		OutIsInString = !OutIsInString;
		OutHasArg = true;
		return false;
	}

	if (*token == TEXT(' ') && !OutIsInString)
	{
		return OutHasArg;
	}

	// if we've made it this far, the token should be added to the argument
	OutCurArg += *token;
	OutHasArg = true;

	return false;
}

FProcHandle FUnixPlatformProcess::CreateProcInternal(const TCHAR* URL, const TCHAR* Parms, bool bLaunchDetached, bool bLaunchHidden, bool bLaunchReallyHidden, uint32* OutProcessID, int32 PriorityModifier, const TCHAR* OptionalWorkingDirectory, void* PipeWriteChild, void* PipeReadChild, void* PipeStdErrChild, bool bInheritHandles)
{
	// @TODO bLaunchHidden bLaunchReallyHidden are not handled

	FString ProcessPath = URL;

	// - If the first character is /, we use the provided absolute path as-is.
	// - If no leading slash, prefer the result of FPaths::ConvertRelativePathToFull if it exists. This is roughly
	//   morally equivalent to prepending the basedir to $PATH (and in turn, Windows' cwd (==basedir) priority).
	// - If there was a (non-leading) slash, and ConvertRelativePathToFull missed, return failure.
	// - If there were no path separators in the input string, allow posix_spawnp to search the $PATH.

	int32 PathSepIdx = INDEX_NONE;
	const bool bInputHasPathSep = ProcessPath.FindChar(TEXT('/'), PathSepIdx);
	bool bAbsolutePath = PathSepIdx == 0;

	if (!bAbsolutePath)
	{
		const FString CandidatePath = FPaths::ConvertRelativePathToFull(ProcessPath);
		if (bInputHasPathSep || FPaths::FileExists(CandidatePath))
		{
			ProcessPath = CandidatePath;
			bAbsolutePath = true;
		}
	}

	// Even if we weren't passed an absolute path, we may have expanded to one above.
	if (bAbsolutePath)
	{
		if (!FPaths::FileExists(ProcessPath))
		{
			UE_LOGF(LogHAL, Error, "FUnixPlatformProcess::CreateProc: File does not exist (%ls)", *ProcessPath);
			return FProcHandle();
		}

		if (!UnixPlatformProcess::AttemptToMakeExecIfNotAlready(ProcessPath))
		{
			UE_LOGF(LogHAL, Error, "FUnixPlatformProcess::CreateProc: File not executable (%ls)", *ProcessPath);
			return FProcHandle();
		}
	}

	if (Parms == nullptr)
	{
		Parms = TEXT("");
	}

	const FString Commandline = FString::Printf(TEXT("\"%s\" %s"), *ProcessPath, Parms);
	UE_LOGF(LogHAL, Verbose, "FUnixPlatformProcess::CreateProc: '%ls'", *Commandline);

	int Argc = 1;
	char* Argv[PlatformProcessLimits::MaxArgvParameters + 1] = { NULL };	// last argument is NULL, hence +1
	Argv[0] = MallocedUtf8FromString(ProcessPath);
	struct CleanupArgvOnExit
	{
		int Argc;
		char** Argv;	// relying on it being long enough to hold Argc elements

		CleanupArgvOnExit(int InArgc, char* InArgv[])
			: Argc(InArgc)
			, Argv(InArgv)
		{}

		~CleanupArgvOnExit()
		{
			for (int Idx = 0; Idx < Argc; ++Idx)
			{
				FMemory::Free(Argv[Idx]);
			}
		}
	} CleanupGuard(Argc, Argv);

	UE_LOGF(LogHAL, Verbose, "FUnixPlatformProcess::CreateProc: ProcessPath = '%ls' Parms = '%ls'", *ProcessPath, Parms);
	FString CurArg; // current argument, during parsing new chars will be appended, will be reused, once the argument is complete
	const TCHAR* CurChar = Parms; // pointer to the current char
	bool IsInString = false; // are we in a string?  if yes, spaces are treated as normal chars
	bool HasArg = false; // do we have a partial argument? CurArg might be empty if Parms contains "". Parms might contain two or more spaces in a row, so not every space indicates the end of an argument
	bool EOL = false;

	// parse Parms and fill Argv
	while (!EOL)
	{
		if (ParseCmdLineToken(CurChar, IsInString, HasArg, CurArg, EOL))
		{
			// if we're still in a string, then quit parsing because we've found a mismatched quote
			// if we can't add the argument, then we've exceeded the maximum number of allowed arguments
			if (!IsInString && !AddCmdLineArgumentTo(Argv, Argc, CurArg))
			{
				break;
			}

			HasArg = false;
			CurArg.Reset(0);
		}

		CurChar++;
	}

	if (IsInString)
	{
		UE_LOGF(LogHAL, Warning, "FUnixPlatformProcess::CreateProc: mismatched quotes in command line (%ls %ls)", *ProcessPath, Parms);
	}

	// we assume PlatformProcessLimits::MaxArgvParameters is >= 1. Since Argc starts at 1 and can never grow larger than PlatformProcessLimits::MaxArgvParameters,
	// we are within the Array
	Argv[Argc] = NULL;

	extern char ** environ;	// provided by libc
	pid_t ChildPid = -1;

	posix_spawnattr_t SpawnAttr;
	posix_spawnattr_init(&SpawnAttr);
	short int SpawnFlags = 0;

	// unmask all signals and set realtime signals to default for children
	// the latter is particularly important for mono, which otherwise will crash attempting to find usable signals
	// (NOTE: setting all signals to default fails)
	sigset_t EmptySignalSet;
	sigemptyset(&EmptySignalSet);
	posix_spawnattr_setsigmask(&SpawnAttr, &EmptySignalSet);
	SpawnFlags |= POSIX_SPAWN_SETSIGMASK;

	sigset_t SetToDefaultSignalSet;
	sigemptyset(&SetToDefaultSignalSet);
	for (int SigNum = SIGRTMIN; SigNum <= SIGRTMAX; ++SigNum)
	{
		sigaddset(&SetToDefaultSignalSet, SigNum);
	}
	posix_spawnattr_setsigdefault(&SpawnAttr, &SetToDefaultSignalSet);
	SpawnFlags |= POSIX_SPAWN_SETSIGDEF;

	// Makes spawned processes have its own unique group id so we can kill the entire group with out killing the parent
	SpawnFlags |= POSIX_SPAWN_SETPGROUP;

	int PosixSpawnErrNo = -1;
	if (PipeWriteChild || PipeReadChild || PipeStdErrChild)
	{
		posix_spawn_file_actions_t FileActions;
		posix_spawn_file_actions_init(&FileActions);

		if (PipeWriteChild)
		{
			const FPipeHandle* PipeReadHandle = reinterpret_cast<const FPipeHandle*>(PipeReadChild);
			const FPipeHandle* PipeWriteHandle = reinterpret_cast<const FPipeHandle*>(PipeWriteChild);

			// If using unique read and write pipes, close the other end of the write pipe
			if (PipeReadChild && PipeWriteHandle->GetPairHandle() != PipeReadHandle->GetHandle())
			{
				posix_spawn_file_actions_addclose(&FileActions, PipeWriteHandle->GetPairHandle());
			}
			posix_spawn_file_actions_adddup2(&FileActions, PipeWriteHandle->GetHandle(), STDOUT_FILENO);
		}

		if (PipeReadChild)
		{
			const FPipeHandle* PipeReadHandle = reinterpret_cast<const FPipeHandle*>(PipeReadChild);
			const FPipeHandle* PipeWriteHandle = reinterpret_cast<const FPipeHandle*>(PipeWriteChild);

			// If using unique read and write pipes, close the other end of the read pipe
			if (PipeWriteChild && PipeReadHandle->GetPairHandle() != PipeWriteHandle->GetHandle())
			{
				posix_spawn_file_actions_addclose(&FileActions, PipeReadHandle->GetPairHandle());
			}
			posix_spawn_file_actions_adddup2(&FileActions, PipeReadHandle->GetHandle(), STDIN_FILENO);
		}

		if (PipeStdErrChild)
		{
			const FPipeHandle* PipeStdErrorHandle = reinterpret_cast<const FPipeHandle*>(PipeStdErrChild);
			posix_spawn_file_actions_adddup2(&FileActions, PipeStdErrorHandle->GetHandle(), STDERR_FILENO);
		}

		posix_spawnattr_setflags(&SpawnAttr, SpawnFlags);
		PosixSpawnErrNo = posix_spawnp(&ChildPid, TCHAR_TO_UTF8(*ProcessPath), &FileActions, &SpawnAttr, Argv, environ);
		posix_spawn_file_actions_destroy(&FileActions);
	}
	else
	{
		// if we don't have any actions to do, use a faster route that will use vfork() instead.
		// This is not just faster, it is crucial when spawning a crash reporter to report a crash due to stack overflow in a thread
		// since otherwise atfork handlers will get called and posix_spawn() will crash (in glibc's __reclaim_stacks()).
		// However, it has its problems, see:
		//		http://ewontfix.com/7/
		//		https://sourceware.org/bugzilla/show_bug.cgi?id=14750
		//		https://sourceware.org/bugzilla/show_bug.cgi?id=14749
		SpawnFlags |= POSIX_SPAWN_USEVFORK;

		posix_spawnattr_setflags(&SpawnAttr, SpawnFlags);
		PosixSpawnErrNo = posix_spawnp(&ChildPid, TCHAR_TO_UTF8(*ProcessPath), nullptr, &SpawnAttr, Argv, environ);
	}
	posix_spawnattr_destroy(&SpawnAttr);

	if (PosixSpawnErrNo != 0)
	{
		UE_LOGF(LogHAL, Error, "FUnixPlatformProcess::CreateProc: posix_spawnp() failed (%d, %s) Path: %ls, Parms: %ls", PosixSpawnErrNo, strerror(PosixSpawnErrNo), *ProcessPath, Parms);
		return FProcHandle();
	}

	// renice the child (subject to race condition).
	// Why this instead of posix_spawn_setschedparam()? 
	// Because posix_spawnattr priority is unusable under Unix due to min/max priority range being [0;0] for the default scheduler
	if (PriorityModifier != 0)
	{
		errno = 0;
		int TheirCurrentPrio = getpriority(PRIO_PROCESS, ChildPid);

		if (errno != 0)
		{
			int ErrNo = errno;
			UE_LOGF(LogHAL, Warning, "FUnixPlatformProcess::CreateProc: could not get child's priority, errno=%d (%ls)",
				ErrNo,
				UTF8_TO_TCHAR(strerror(ErrNo))
			);
			
			// proceed anyway...
			TheirCurrentPrio = 0;
		}

		rlimit PrioLimits;
		int MaxPrio = 0;
		if (getrlimit(RLIMIT_NICE, &PrioLimits) == -1)
		{
			int ErrNo = errno;
			UE_LOGF(LogHAL, Warning, "FUnixPlatformProcess::CreateProc: could not get priority limits (RLIMIT_NICE), errno=%d (%ls)",
				ErrNo,
				UTF8_TO_TCHAR(strerror(ErrNo))
			);

			// proceed anyway...
		}
		else
		{
			MaxPrio = 20 - PrioLimits.rlim_cur;
		}

		int NewPrio = TheirCurrentPrio;
		// This should be in sync with Mac/Windows and Also SetThreadPriority in Unix thread.
		// The single most important use of that is setting "below normal" priority to ShaderCompileWorker (PrioModifier == -1).
		// If SCW is run with too low a priority, shader compilation will be longer than needed.
		int PrioChange = 0;
		if (PriorityModifier > 0)
		{
			// decrease the nice value - will perhaps fail, it's up to the user to run with proper permissions
			PrioChange = (PriorityModifier == 1) ? -10 : -15;
		}
		else if (PriorityModifier < 0)
		{
			PrioChange = (PriorityModifier == -1) ? 5 : 10;
		}
		NewPrio += PrioChange;

		// cap to [RLIMIT_NICE, 19]
		NewPrio = FMath::Min(19, NewPrio);
		NewPrio = FMath::Max(MaxPrio, NewPrio);	// MaxPrio is actually the _lowest_ numerically priority

		if (setpriority(PRIO_PROCESS, ChildPid, NewPrio) == -1)
		{
			int ErrNo = errno;
			UE_LOGF(LogHAL, Warning, "FUnixPlatformProcess::CreateProc: could not change child's priority (nice value) from %d to %d, errno=%d (%ls)",
				TheirCurrentPrio, NewPrio,
				ErrNo,
				UTF8_TO_TCHAR(strerror(ErrNo))
			);
		}
		else
		{
			UE_LOGF(LogHAL, Verbose, "Changed child's priority (nice value) to %d (change from %d)", NewPrio, TheirCurrentPrio);
		}
	}

	else
	{
		UE_LOGF(LogHAL, Verbose, "FUnixPlatformProcess::CreateProc: spawned child %d", ChildPid);
	}

	if (OutProcessID)
	{
		*OutProcessID = ChildPid;
	}

	// [RCL] 2015-03-11 @FIXME: is bLaunchDetached usable when determining whether we're in 'fire and forget' mode? This doesn't exactly match what bLaunchDetached is used for.
	return FProcHandle(new FProcState(ChildPid, bLaunchDetached));
}

/**
 * Read the start time of a process from /proc/<pid>/stat field 22 (starttime).
 * Returns clock ticks since boot, or 0 on failure. PID + starttime uniquely
 * identifies a process instance even across PID reuse.
 */
static uint64 ReadProcessStartTime(pid_t Pid)
{
	// "/proc/" (6) + max PID digits (7) + "/stat" (5) + '\0' (1) = 19; 32 is generous
	char StatPath[32];
	FCStringAnsi::Sprintf(StatPath, "/proc/%d/stat", static_cast<int>(Pid));

	FILE* StatFile = fopen(StatPath, "r");
	if (StatFile == nullptr)
	{
		return 0;
	}

	// /proc/<pid>/stat is a single line; 1024 is generous for any process name
	char Line[1024];
	bool bReadOk = (fgets(Line, sizeof(Line), StatFile) != nullptr);
	fclose(StatFile);

	if (!bReadOk)
	{
		return 0;
	}

	// Field 2 is (comm) which can contain spaces and parentheses.
	// Find the last ')' to safely skip past it, then count fields from field 3.
	const char* CloseParen = strrchr(Line, ')');
	if (CloseParen == nullptr)
	{
		return 0;
	}

	// Field 22 (starttime) is 20 fields after field 2.
	// After ')' there is a space, then field 3 starts.
	const char* Pos = CloseParen + 1;
	for (int Field = 3; Field < 22; ++Field)
	{
		while (*Pos == ' ')
		{
			++Pos;
		}
		if (*Pos == '\0')
		{
			return 0;
		}
		while (*Pos != ' ' && *Pos != '\0')
		{
			++Pos;
		}
	}
	while (*Pos == ' ')
	{
		++Pos;
	}

	if (*Pos == '\0')
	{
		return 0;
	}

	return static_cast<uint64>(strtoull(Pos, nullptr, 10));
}

/**
 * Validate that an OpenProcess handle still refers to the same process.
 * For CreateProc handles (ProcInfo != nullptr), always succeeds.
 * For OpenProcess handles, compares the stored starttime against the current
 * value in /proc to detect PID reuse. Sets OutPid on success.
 */
static bool ValidateOpenProcessHandle(const FProcHandle& Handle, pid_t& OutPid)
{
	if (Handle.GetProcessInfo() != nullptr)
	{
		OutPid = Handle.GetProcessInfo()->GetProcessId();
		return true;
	}

	if (Handle.Get() == -1)
	{
		return false;
	}

	OutPid = static_cast<pid_t>(Handle.Get());

	// If we didn't capture a starttime (e.g. /proc was unavailable), skip validation
	if (Handle.OpenedStartTime == 0)
	{
		return true;
	}

	uint64 CurrentStartTime = ReadProcessStartTime(OutPid);
	if (CurrentStartTime == 0)
	{
		// Process no longer exists (can't read /proc/<pid>/stat)
		return false;
	}

	if (CurrentStartTime != Handle.OpenedStartTime)
	{
		UE_LOGF(LogHAL, Warning, "OpenProcess handle for pid %d detected PID reuse (starttime %" UINT64_FMT " vs %" UINT64_FMT "). Ignoring stale handle.",
			static_cast<int32>(OutPid), Handle.OpenedStartTime, CurrentStartTime);
		return false;
	}

	return true;
}

FProcHandle FUnixPlatformProcess::OpenProcess(uint32 ProcessID)
{
	pid_t Pid = static_cast< pid_t >(ProcessID);

	// check if actually running
	int KillResult = kill(Pid, 0);	// no actual signal is sent
	check(KillResult != -1 || errno != EINVAL);

	// errno == EPERM: don't have permissions to send signal
	// errno == ESRCH: proc doesn't exist
	bool bIsRunning = (KillResult == 0);
	FProcHandle Handle(bIsRunning ? Pid : -1);
	if (bIsRunning)
	{
		Handle.OpenedStartTime = ReadProcessStartTime(Pid);
	}
	return Handle;
}

/** Initialization constructor. */
FProcState::FProcState(pid_t InProcessId, bool bInFireAndForget)
	:	ProcessId(InProcessId)
	,	bIsRunning(true)  // assume it is
	,	bHasBeenWaitedFor(false)
	,	ReturnCode(-1)
	,	bFireAndForget(bInFireAndForget)
{
}

FProcState::~FProcState()
{
	if (!bFireAndForget)
	{
		// If not in 'fire and forget' mode, try to catch the common problems that leave zombies:
		// - We don't want to close the handle of a running process as with our current scheme this will certainly leak a zombie.
		// - Nor we want to leave the handle unwait()ed for.
		
		if (bIsRunning)
		{
			// Warn the users before going into what may be a very long block
			UE_LOGF(LogHAL, Warning, "Closing a process handle while the process (pid=%d) is still running - we will block until it exits to prevent a zombie",
				GetProcessId()
			);
		}
		else if (!bHasBeenWaitedFor)	// if child is not running, but has not been waited for, still communicate a problem, but we shouldn't be blocked for long in this case.
		{
			UE_LOGF(LogHAL, Warning, "Closing a process handle of a process (pid=%d) that has not been wait()ed for - will wait() now to reap a zombie",
				GetProcessId()
			);
		}

		Wait();	// will exit immediately if everything is Ok
	}
	else if (IsRunning())
	{
		// warn about leaking a thread ;/
		UE_LOGF(LogHAL, Verbose, "Process (pid=%d) is still running - we will reap it in a waiter thread, but the thread handle is going to be leaked.",
				 GetProcessId()
			);

		FChildWaiterThread * WaiterRunnable = new FChildWaiterThread(GetProcessId());
		// [RCL] 2015-03-11 @FIXME: do not leak
		FRunnableThread * WaiterThread = FRunnableThread::Create(WaiterRunnable, *FString::Printf(TEXT("waitpid(%d)"), GetProcessId()), 32768 /* needs just a small stack */, TPri_BelowNormal);
	}
}

bool FProcState::IsRunning()
{
	if (bIsRunning)
	{
		check(!bHasBeenWaitedFor);	// check for the sake of internal consistency

		// check if actually running
		int KillResult = kill(GetProcessId(), 0);	// no actual signal is sent
		check(KillResult != -1 || errno != EINVAL);

		bIsRunning = (KillResult == 0 || (KillResult == -1 && errno == EPERM));

		// additional check if it's a zombie
		if (bIsRunning)
		{
			for(;;)	// infinite loop in case we get EINTR and have to repeat
			{
				siginfo_t SignalInfo;
				SignalInfo.si_pid = 0;	// if remains 0, treat as child was not waitable (i.e. was running)
				if (waitid(P_PID, GetProcessId(), &SignalInfo, WEXITED | WNOHANG | WNOWAIT))
				{
					if (errno != EINTR)
					{
						int ErrNo = errno;
						UE_LOGF(LogHAL, Fatal, "FUnixPlatformProcess::WaitForProc: waitid for pid %d failed (errno=%d, %ls)", 
							static_cast< int32 >(GetProcessId()), ErrNo, UTF8_TO_TCHAR(strerror(ErrNo)));
						break;	// exit the loop if for some reason Fatal log (above) returns
					}
				}
				else
				{
					bIsRunning = ( SignalInfo.si_pid != GetProcessId() );
					break;
				}
			}
		}

		// If child is a zombie, wait() immediately to free up kernel resources. Higher level code
		// (e.g. shader compiling manager) can hold on to handle of no longer running process for longer,
		// which is a dubious, but valid behavior. We don't want to keep zombie around though.
		if (!bIsRunning)
		{
			UE_LOGF(LogHAL, Verbose, "Child %d is no longer running (zombie), Wait()ing immediately.", GetProcessId() );
			Wait();
		}
	}

	return bIsRunning;
}

bool FProcState::GetReturnCode(int32* ReturnCodePtr)
{
	check(!bIsRunning || !"You cannot get a return code of a running process");
	if (!bHasBeenWaitedFor)
	{
		Wait();
	}

	if (ReturnCode != -1)
	{
		if (ReturnCodePtr != NULL)
		{
			*ReturnCodePtr = ReturnCode;
		}
		return true;
	}

	return false;
}

void FProcState::Wait()
{
	if (bHasBeenWaitedFor)
	{
		return;	// we could try waitpid() another time, but why
	}

	for(;;)	// infinite loop in case we get EINTR and have to repeat
	{
		siginfo_t SignalInfo;
		if (waitid(P_PID, GetProcessId(), &SignalInfo, WEXITED))
		{
			if (errno != EINTR)
			{
				int ErrNo = errno;
				UE_LOGF(LogHAL, Fatal, "FUnixPlatformProcess::WaitForProc: waitid for pid %d failed (errno=%d, %ls)", 
					static_cast< int32 >(GetProcessId()), ErrNo, UTF8_TO_TCHAR(strerror(ErrNo)));
				break;	// exit the loop if for some reason Fatal log (above) returns
			}
		}
		else
		{
			check(SignalInfo.si_pid == GetProcessId());

			ReturnCode = (SignalInfo.si_code == CLD_EXITED) ? SignalInfo.si_status : -1;
			bHasBeenWaitedFor = true;
			bIsRunning = false;	// set in advance
			UE_LOGF(LogHAL, Verbose, "Child %d's return code is %d.", GetProcessId(), ReturnCode);
			break;
		}
	}
}

bool FUnixPlatformProcess::IsProcRunning( FProcHandle & ProcessHandle )
{
	FProcState * ProcInfo = ProcessHandle.GetProcessInfo();
	if (ProcInfo)
	{
		return ProcInfo->IsRunning();
	}

	pid_t Pid;
	if (!ValidateOpenProcessHandle(ProcessHandle, Pid))
	{
		return false;
	}

	int KillResult = kill(Pid, 0);
	check(KillResult != -1 || errno != EINVAL);
	return (KillResult == 0);
}

void FUnixPlatformProcess::WaitForProc( FProcHandle & ProcessHandle )
{
	FProcState * ProcInfo = ProcessHandle.GetProcessInfo();
	if (ProcInfo)
	{
		ProcInfo->Wait();
	}
	else
	{
		pid_t Pid;
		if (!ValidateOpenProcessHandle(ProcessHandle, Pid))
		{
			return;
		}

		// Poll /proc/<pid>/status until the process disappears or becomes a zombie.
		// "/proc/" (6) + max PID digits (7, up to 4194304) + "/status" (7) + '\0' (1) = 21; 48 is generous
		char StatusPath[48];
		FCStringAnsi::Sprintf(StatusPath, "/proc/%d/status", static_cast<int>(Pid));
		while (true)
		{
			FILE* StatusFile = fopen(StatusPath, "r");
			if (StatusFile == nullptr)
			{
				break; // process no longer exists
			}

			bool bIsZombie = false;
			// Longest line in /proc/pid/status is typically under 80 chars; 256 is generous
			char Line[256];
			while (fgets(Line, sizeof(Line), StatusFile) != nullptr)
			{
				if (strncmp(Line, "State:", 6) == 0)
				{
					// State line format: "State:\t<char> (<description>)"
					const char* StateChar = Line + 6;
					while (*StateChar != '\0' && (*StateChar == '\t' || *StateChar == ' '))
					{
						++StateChar;
					}
					bIsZombie = (*StateChar == 'Z');
					break;
				}
			}
			fclose(StatusFile);

			if (bIsZombie)
			{
				break;
			}
			FPlatformProcess::Sleep(0.05f);
		}
	}
}

void FUnixPlatformProcess::CloseProc(FProcHandle & ProcessHandle)
{
	// dispose of both handle and process info
	FProcState * ProcInfo = ProcessHandle.GetProcessInfo();
	ProcessHandle.Reset();

	delete ProcInfo;
}

void FUnixPlatformProcess::TerminateProc( FProcHandle & ProcessHandle, bool KillTree )
{
	FProcState* ProcInfo = ProcessHandle.GetProcessInfo();

	pid_t RootPid = -1;
	if (ProcInfo != nullptr)
	{
		RootPid = ProcInfo->GetProcessId();
	}
	else if (!ValidateOpenProcessHandle(ProcessHandle, RootPid))
	{
		return;
	}

	if (KillTree && RootPid != -1)
	{
		// Iteratively collect all descendant PIDs via BFS, seeded by one /proc scan per pass.
		// Two passes are used: the first finds the initial tree; the second catches any processes
		// that were forked after the first scan but before their parent received SIGSTOP.
		TArray<pid_t> ToKill;
		ToKill.Add(RootPid);
		if (kill(RootPid, SIGSTOP) == -1 && errno != ESRCH)
		{
			int ErrNo = errno;
			UE_LOGF(LogHAL, Warning, "TerminateProc KillTree: SIGSTOP to root pid %d failed (errno=%d, %s)",
				static_cast<int32>(RootPid), ErrNo, strerror(ErrNo));
		}

		// ESRCH always needs to be handled as processes can end before we finish each of the passes/stages
		for (int32 Pass = 0; Pass < 2; ++Pass)
		{
			// Capture process id and parent id
			TArray<TPair<pid_t, pid_t>> AllProcs;

			DIR* ProcDir = opendir("/proc");
			if (ProcDir == nullptr)
			{
				int ErrNo = errno;
				UE_LOGF(LogHAL, Warning, "TerminateProc KillTree: unable to open /proc (errno=%d, %s) -- will kill root process (pid %d) but cannot discover child processes.",
					ErrNo, strerror(ErrNo), static_cast<int32>(RootPid));
				break; // second pass would also fail
			}
			else
			{
				struct dirent* Entry;
				while ((Entry = readdir(ProcDir)) != nullptr)
				{
					const char* Name = Entry->d_name;
					if (!FCStringAnsi::IsNumeric(Name))
					{
						continue;
					}

					pid_t Pid = static_cast<pid_t>(FCStringAnsi::Atoi(Name));
					if (Pid == 0)
					{
						continue;
					}

					// "/proc/" (6) + max PID digits (7) + "/status" (7) + '\0' (1) = 21; 48 is generous
					char StatusPath[48];
					FCStringAnsi::Sprintf(StatusPath, "/proc/%d/status", static_cast<int>(Pid));

					FILE* StatusFile = fopen(StatusPath, "r");
					if (StatusFile == nullptr)
					{
						continue;
					}

					pid_t ParentPid = -1;
					char Line[128];
					while (fgets(Line, sizeof(Line), StatusFile) != nullptr)
					{
						if (strncmp(Line, "PPid:", 5) == 0)
						{
							ParentPid = static_cast<pid_t>(FCStringAnsi::Atoi(Line + 5));
							break;
						}
					}
					fclose(StatusFile);

					if (ParentPid >= 0)
					{
						AllProcs.Emplace(Pid, ParentPid);
					}
				}
				closedir(ProcDir);
			}

			// BFS from every pid already in ToKill to discover new descendants.
			// SIGSTOP each process immediately upon discovery to prevent it from forking.
			for (int32 QueueIdx = 0; QueueIdx < ToKill.Num(); ++QueueIdx)
			{
				pid_t CurrentPid = ToKill[QueueIdx];
				for (const TPair<pid_t, pid_t>& Proc : AllProcs)
				{
					if (Proc.Value == CurrentPid && !ToKill.Contains(Proc.Key))
					{
						ToKill.Add(Proc.Key);
						if (kill(Proc.Key, SIGSTOP) == -1 && errno != ESRCH)
						{
							int ErrNo = errno;
							UE_LOGF(LogHAL, Warning, "TerminateProc KillTree: SIGSTOP to pid %d failed (errno=%d, %s)",
								static_cast<int32>(Proc.Key), ErrNo, strerror(ErrNo));
						}
					}
				}
			}
		}

		// Send SIGTERM to all the processes as a group before SIGCONT (which processes the SIGTERM) so the whole batch is kept together
		for (int32 Index = ToKill.Num() - 1; Index >= 0; --Index)
		{
			if (kill(ToKill[Index], SIGTERM) == -1 && errno != ESRCH)
			{
				int ErrNo = errno;
				UE_LOGF(LogHAL, Warning, "TerminateProc KillTree: SIGTERM to pid %d failed (errno=%d, %s)",
					static_cast<int32>(ToKill[Index]), ErrNo, strerror(ErrNo));
			}
		}

		// Resume all processes so the queued SIGTERM is delivered and they can exit gracefully.
		for (int32 Index = ToKill.Num() - 1; Index >= 0; --Index)
		{
			if (kill(ToKill[Index], SIGCONT) == -1 && errno != ESRCH)
			{
				int ErrNo = errno;
				UE_LOGF(LogHAL, Warning, "TerminateProc KillTree: SIGCONT to pid %d failed (errno=%d, %s)",
					static_cast<int32>(ToKill[Index]), ErrNo, strerror(ErrNo));
			}
		}
		return;
	}

	if (RootPid != -1)
	{
		int KillResult = kill(RootPid, SIGTERM);
		check(KillResult != -1 || errno != EINVAL);
	}
}

static FDelegateHandle OnEndFrameHandle;

/*
 * WaitAndFork on Unix
 *
 * This is a function that halts execution and waits for signals to cause forked processes to be created and continue execution.
 * The parent process will return when IsEngineExitRequested() is true. SIGRTMIN+1 is used to cause a fork to happen.
 * Optionally, the parent process will also return if any of the children close with an exit code equal to WAIT_AND_FORK_PARENT_SHUTDOWN_EXIT_CODE if it is set to non-zero.
 * If sigqueue is used, the payload int will be split into the upper and lower uint16 values. The upper value is a "cookie" and the
 *     lower value is an "index". These two values will be used to name the process using the pattern DS-<cookie>-<index>. This name
 *     can be used to uniquely discover the process that was spawned.
 * If -NumForks=x is suppled on the command line, x forks will be made when the function is called, and if any forked processes close for any reason, they will be reopened
 * If -WaitAndForkCmdLinePath=Foo is suppled, the command line parameters of the child processes will be filled out with the contents
 *     of files found in the directory referred to by Foo, where the child's "index" is the name of the file to be read in the directory.
 * If -WaitAndForkRequireResponse is on the command line, child processes will not proceed after being spawned until a SIGRTMIN+2 signal is sent to them.
 */
FGenericPlatformProcess::EWaitAndForkResult FUnixPlatformProcess::WaitAndFork()
{
#define WAIT_AND_FORK_QUEUE_LENGTH 4096
#ifndef WAIT_AND_FORK_PARENT_SHUTDOWN_EXIT_CODE
	#define WAIT_AND_FORK_PARENT_SHUTDOWN_EXIT_CODE 0
#endif
#ifndef WAIT_AND_FORK_RESPONSE_TIMEOUT_EXIT_CODE
	#define WAIT_AND_FORK_RESPONSE_TIMEOUT_EXIT_CODE 1
#endif
	
	// Only works in -nothreading mode for now (probably best this way)
	if (FPlatformProcess::SupportsMultithreading())
	{
		return EWaitAndForkResult::Error;
	}

	struct FForkSignalData
	{
		FForkSignalData() = default;
		FForkSignalData(int32 InSignal, double InSeconds) : SignalValue(InSignal), TimeSeconds(InSeconds) {}

		int32 SignalValue = 0;
		double TimeSeconds = 0.0;
	};

	static TCircularQueue<FForkSignalData> WaitAndForkSignalQueue(WAIT_AND_FORK_QUEUE_LENGTH);

	// If we asked to fork up front without the need to send signals, just push the fork requests on the queue and we will refork them if they close
	// This is mostly used in cases where there is no external process sending signals to this process to create forks and is a simple way to start or test
	int32 NumForks = 0;
	FParse::Value(FCommandLine::Get(), TEXT("-NumForks="), NumForks);
	if (NumForks > 0)
	{
		for (int32 ForkIdx = 0; ForkIdx < NumForks; ++ForkIdx)
		{
			WaitAndForkSignalQueue.Enqueue(FForkSignalData(ForkIdx + 1, FPlatformTime::Seconds()));
		}
	}

	// If we asked to fill out command line parameters from files on disk, read the folder that contains the parameters
	FString ChildParametersPath;
	FParse::Value(FCommandLine::Get(), TEXT("-WaitAndForkCmdLinePath="), ChildParametersPath);
	if (!ChildParametersPath.IsEmpty())
	{
		bool bDirExists = IFileManager::Get().DirectoryExists(*ChildParametersPath);
		if (!bDirExists)
		{
			UE_LOGF(LogFork, Fatal, "Path referred to by -WaitAndForkCmdLinePath does not exist: %ls", *ChildParametersPath);
		}
	}

	// If we are asked to wait for a response signal, keep track of that here so we can behave differently in children.
	const bool bRequireResponseSignal = FParse::Param(FCommandLine::Get(), TEXT("WaitAndForkRequireResponse"));

	double WaitAndForkResponseTimeout = -1.0;
	FParse::Value(FCommandLine::Get(), TEXT("-WaitAndForkResponseTimeout="), WaitAndForkResponseTimeout);
	if (WaitAndForkResponseTimeout > 0.0)
	{
		UE_LOGF(LogFork, Log, "WaitAndFork setting WaitAndForkResponseTimeout to %0.2f seconds.", WaitAndForkResponseTimeout);
	}

	FCoreDelegates::OnParentBeginFork.Broadcast();

	// Set up a signal handler for the signal to fork()
	{
		struct sigaction Action;
		FMemory::Memzero(Action);
		sigfillset(&Action.sa_mask);
		Action.sa_flags = SA_SIGINFO | SA_RESTART | SA_ONSTACK;
		Action.sa_sigaction = [](int32 Signal, siginfo_t* Info, void* Context) {
			if (Signal == WAIT_AND_FORK_QUEUE_SIGNAL && Info)
			{
				WaitAndForkSignalQueue.Enqueue(FForkSignalData(Info->si_value.sival_int, FPlatformTime::Seconds()));
			}
		};
		sigaction(WAIT_AND_FORK_QUEUE_SIGNAL, &Action, nullptr);
	}

	UE_LOGF(LogFork, Log, "   *** WaitAndFork awaiting signal %d to process pid %d create child processes... ***", WAIT_AND_FORK_QUEUE_SIGNAL, FPlatformProcess::GetCurrentProcessId());
	GLog->Flush();

	struct FMemoryStatsHolder
	{
		double AvailablePhysical = 0.0;
		double PeakUsedPhysical = 0.0;
		double PeakUsedVirtual = 0.0;

		constexpr double ByteToMiB(uint64 InBytes) { return static_cast<double>(InBytes) / (1024.0 * 1024.0); }

		FMemoryStatsHolder(const FPlatformMemoryStats& PlatformStats)
			: AvailablePhysical(ByteToMiB(PlatformStats.AvailablePhysical))
			, PeakUsedPhysical(ByteToMiB(PlatformStats.PeakUsedPhysical))
			, PeakUsedVirtual(ByteToMiB(PlatformStats.PeakUsedVirtual))
		{ }
	};

	TOptional<FMemoryStatsHolder> PreviousMasterMemStats;
	
	if (UnixPlatformProcess::GLogMemoryStatsWhenForking)
	{
		PreviousMasterMemStats = FMemoryStatsHolder(FPlatformMemory::GetStats());
	}

	EWaitAndForkResult RetVal = EWaitAndForkResult::Parent;
	struct FPidAndSignal
	{
		pid_t Pid;
		int32 SignalValue;

		FPidAndSignal() : SignalValue(0) {}
		FPidAndSignal(pid_t InPid, int32 InSignalValue) : Pid(InPid), SignalValue(InSignalValue) {}
	};
	TArray<FPidAndSignal> AllChildren;
	AllChildren.Reserve(1024); // Sized to be big enough that it probably wont reallocate, but its not the end of the world if it does.
	bool ChildBreakOut = false;	// due to batch forking support, child processes can no longer use 'break' to exit the while loops
	while ( !ChildBreakOut && !IsEngineExitRequested() )
	{
		BeginExitIfRequested();

		float SpawnDelay = FMath::Max(0, UnixPlatformProcess::CVarForkParentSpawnDelay.GetValueOnAnyThread());
		int BatchCount = FMath::Max(1,UnixPlatformProcess::CVarForkBatchSize.GetValueOnAnyThread());
		FForkSignalData SignalData;
		if ( !WaitAndForkSignalQueue.IsEmpty() )
		{
			UE_LOGF(LogFork, Log, "[Parent] WaitAndForkSignalQueue has %d signals to execute. Sleeping for %f secs before processing", WaitAndForkSignalQueue.Count(), SpawnDelay);

			// Sleep for a short while to avoid spamming new processes to the OS all at once
			FPlatformProcess::Sleep(SpawnDelay);

			
			while ( !ChildBreakOut && !IsEngineExitRequested() && BatchCount > 0 )
			{
				// need to hold up the fork signal while we Dequeue since there's no way for the queue itself to guard against races due to a signal handler
				sigset_t OldSigMask, NewSigMask;
				sigfillset(&NewSigMask);	
				int MaskResult = sigprocmask(SIG_BLOCK, &NewSigMask, &OldSigMask);
				bool HasWaitingSignals = WaitAndForkSignalQueue.Dequeue(SignalData);
				// unblock fork signal
				if ( MaskResult == 0 )
				{
					// don't try to revert mask if it wasn't successfully changed above
					sigprocmask(SIG_SETMASK, &OldSigMask, nullptr);
				}

				if (!HasWaitingSignals)
				{
					// all signals already processed, return to sleep and then wait for a new signal
					break;
				}

				uint16 Cookie = (SignalData.SignalValue >> 16) & 0xffff;
				uint16 ChildIdx = SignalData.SignalValue & 0xffff;

				FCoreDelegates::OnParentPreFork.Broadcast();

				UE_LOGF(LogFork, Log, "[Parent] WaitAndFork processing child request %04hx-%04hx received at: %ls", Cookie, ChildIdx, *UnixPlatformProcess::GetReadableTime(FPlatformTime::Seconds() - SignalData.TimeSeconds));

				if (UnixPlatformProcess::GLogMemoryStatsWhenForking)
				{
					FMemoryStatsHolder CurrentMasterMemStats(FPlatformMemory::GetStats());
					UE_LOGF(LogFork, Log, "[Parent] MemoryStats PreFork: AvailablePhysical: %.02fMiB (%+.02fMiB), PeakPhysical: %.02fMiB, PeakVirtual: %.02fMiB",
						CurrentMasterMemStats.AvailablePhysical, (CurrentMasterMemStats.AvailablePhysical - PreviousMasterMemStats->AvailablePhysical),
						CurrentMasterMemStats.PeakUsedPhysical,
						CurrentMasterMemStats.PeakUsedVirtual
					);
					PreviousMasterMemStats = CurrentMasterMemStats;
				}

				// Make sure there are no pending messages in the log.
				GLog->Flush();

				// This should be the very last thing we do before forking for optimal interaction with GMalloc
				FForkProcessHelper::LowLevelPreFork();
				// ******** The fork happens here! ********
				pid_t ChildPID = fork();
				// ******** The fork happened! This is now either the parent process or the new child process ********

				if (ChildPID == -1)
				{
					// Error handling
					// We could return with an error code here, but instead it is somewhat better to log out an error and continue since this loop is supposed to be stable.
					// Fork errors may include hitting process limits or other environmental factors so we will just report the issue since the environmental factor can be
					// fixed while the process is still running.
					int ErrNo = errno;
					UE_LOGF(LogFork, Error, "WaitAndFork failed to fork! fork() error:%d", ErrNo);
				}
				else if (ChildPID == 0)
				{
					// This should be the very first thing we do after forking for optimal interaction with GMalloc
					FForkProcessHelper::LowLevelPostForkChild(ChildIdx);

					if (FPlatformMemory::HasForkPageProtectorEnabled())
					{
						UE::FForkPageProtector::OverrideGMalloc();
						UE::FForkPageProtector::Get().ProtectMemoryRegions();
					}

					// Close the log state we inherited from our parent
					GLog->TearDown();

					// Update GGameThreadId
					FUnixTLS::ClearThreadIdTLS();
					GGameThreadId = FUnixTLS::GetCurrentThreadId();

					// Fix the command line, if a path to command line parameters was specified
					if (!ChildParametersPath.IsEmpty() && ChildIdx > 0)
					{
						FString NewCmdLine;
						const FString CmdLineFilename = ChildParametersPath / FString::Printf(TEXT("%d"), ChildIdx);
						FFileHelper::LoadFileToString(NewCmdLine, *CmdLineFilename);
						if (!NewCmdLine.IsEmpty())
						{
							FCommandLine::Set(*NewCmdLine);
						}
						else
						{
							UE_LOGF(LogFork, Error, "[Child] WaitAndFork child %04hx-%04hx failed to set command line from: %ls", Cookie, ChildIdx, *CmdLineFilename);
						}
					}

					// Start up the log again
					FPlatformOutputDevices::SetupOutputDevices();
					GLog->SetCurrentThreadAsPrimaryThread();


					// Set the process name, if specified
					if (ChildIdx > 0)
					{
						if (prctl(PR_SET_NAME, TCHAR_TO_UTF8(*FString::Printf(TEXT("DS-%04hx-%04hx"), Cookie, ChildIdx))) != 0)
						{
							int ErrNo = errno;
							UE_LOGF(LogFork, Fatal, "[Child] WaitAndFork failed to set process name with prctl! error:%d", ErrNo);
						}
					}

					// Need to remove the child process from the responsibility of keeping track of a valid sibling process.
					UnixCrashReporterTracker::RemoveValidCrashReportTickerForChildProcess();

					// If requested, now wait for a SIGRTMIN+2 signal before continuing execution.
					if (bRequireResponseSignal && (ChildIdx == 0 || ChildIdx > NumForks))
					{
						static bool bResponseReceived = false;
						struct sigaction Action;
						FMemory::Memzero(Action);
						sigfillset(&Action.sa_mask);
						Action.sa_flags = SA_SIGINFO | SA_RESTART | SA_ONSTACK;
						Action.sa_sigaction = [](int32 Signal, siginfo_t* Info, void* Context) {
							if (Signal == WAIT_AND_FORK_RESPONSE_SIGNAL)
							{
								bResponseReceived = true;
							}
							};
						sigaction(WAIT_AND_FORK_RESPONSE_SIGNAL, &Action, nullptr);

						const double StartChildWaitSeconds = FPlatformTime::Seconds();

						UE_LOGF(LogFork, Log, "[Child] WaitAndFork child %04hx-%04hx waiting for signal %d to proceed.", Cookie, ChildIdx, WAIT_AND_FORK_RESPONSE_SIGNAL);

						while (!IsEngineExitRequested() && !bResponseReceived)
						{
							UE_LOGF(LogFork, Verbose, "[Child] WaitAndFork child %04hx-%04hx signal %d not received. Sleeping for 1sec.", Cookie, ChildIdx, WAIT_AND_FORK_RESPONSE_SIGNAL);

							FPlatformProcess::Sleep(1);

							// Check to see how long we've been waiting and if we should time out.
							if ((WaitAndForkResponseTimeout > 0.0) && ((FPlatformTime::Seconds() - StartChildWaitSeconds) > WaitAndForkResponseTimeout))
							{
								UE_LOGF(LogFork, Error, "[Child] WaitAndFork child %04hx-%04hx has exceeded WAIT_AND_FORK_RESPONSE_SIGNAL timeout", Cookie, ChildIdx);
								FPlatformMisc::RequestExitWithStatus(true, WAIT_AND_FORK_RESPONSE_TIMEOUT_EXIT_CODE);
								break;
							}
						}

						FMemory::Memzero(Action);
						sigaction(WAIT_AND_FORK_RESPONSE_SIGNAL, &Action, nullptr);
					}

					UE_LOGF(LogFork, Log, "[Child] WaitAndFork child process %04hx-%04hx has started with pid %d.", Cookie, ChildIdx, GetCurrentProcessId());
					FApp::PrintStartupLogMessages();

					OnEndFrameHandle = FCoreDelegates::OnEndFrame.AddStatic(FUnixPlatformProcess::OnChildEndFramePostFork);
					FCoreDelegates::OnPostFork.Broadcast(EForkProcessRole::Child);

#if UE_CHECK_LARGE_ALLOCATIONS
					if (CVarEnableLargeAllocationChecksAfterFork.GetValueOnAnyThread())
					{
						UE::Memory::Private::CVarEnableLargeAllocationChecks->Set(true);
					}
#endif

					// Children break out of the loop and return
					RetVal = EWaitAndForkResult::Child;
					ChildBreakOut = true;
				}
				else
				{
					// This should be the very first thing we do after forking for optimal interaction with GMalloc
					FForkProcessHelper::LowLevelPostForkParent();

					// Parent
					AllChildren.Emplace(ChildPID, SignalData.SignalValue);

					FCoreDelegates::OnPostFork.Broadcast(EForkProcessRole::Parent);

					UE_LOGF(LogFork, Log, "[Parent] WaitAndFork Successfully processed request %04hx-%04hx, made a child with pid %d! Total number of children: %d.", Cookie, ChildIdx, ChildPID, AllChildren.Num());
				}
				BatchCount--;
			}
		}
		else
		{
			UE_LOGF(LogFork, Verbose, "[Parent] SignalQueue is empty. Sleeping for %f secs", UnixPlatformProcess::GParentSleepDurationInSec);

			// No signal to process. Sleep for a bit and do some bookkeeping.
			FPlatformProcess::Sleep(UnixPlatformProcess::GParentSleepDurationInSec);

			// Trim terminated children
			for (int32 ChildIdx = AllChildren.Num() - 1; ChildIdx >= 0; --ChildIdx)
			{
				const FPidAndSignal& ChildPidAndSignal = AllChildren[ChildIdx];

				int32 Status = 0;
				pid_t WaitResult = waitpid(ChildPidAndSignal.Pid, &Status, WNOHANG);
				if (WaitResult == -1)
				{
					int32 ErrNo = errno;
					UE_LOGF(LogFork, Log, "[Parent] WaitAndFork unknown error while querying existance of child %d. Error:%d", ChildPidAndSignal.Pid, ErrNo);
				}
				else if (WaitResult != 0)
				{
					int32 ExitCode = WIFEXITED(Status) ? WEXITSTATUS(Status) : 0;
					if (ExitCode != 0 && ExitCode == WAIT_AND_FORK_PARENT_SHUTDOWN_EXIT_CODE)
					{
						UE_LOGF(LogFork, Log, "[Parent] WaitAndFork child %d exited with return code %d, indicating that the parent process should shut down. Shutting down...", ChildPidAndSignal.Pid, WAIT_AND_FORK_PARENT_SHUTDOWN_EXIT_CODE);
						RequestEngineExit(TEXT("Unix Child has exited"));
					}
					else if (NumForks > 0 && ChildPidAndSignal.SignalValue > 0 && ChildPidAndSignal.SignalValue <= NumForks)
					{
						UE_LOGF(LogFork, Log, "[Parent] WaitAndFork child %d missing. This was NumForks child %d. Relaunching...", ChildPidAndSignal.Pid, ChildPidAndSignal.SignalValue);
						WaitAndForkSignalQueue.Enqueue(FForkSignalData(ChildPidAndSignal.SignalValue, FPlatformTime::Seconds()));
					}
					else
					{
						UE_LOGF(LogFork, Log, "[Parent] WaitAndFork child %d missing. Removing from children list...", ChildPidAndSignal.Pid);
					}

					AllChildren.RemoveAt(ChildIdx, EAllowShrinking::No);
				}
			}
		}
	}

	// Clean up the queue signal handler from earlier.
	{
		struct sigaction Action;
		FMemory::Memzero(Action);
		sigaction(WAIT_AND_FORK_QUEUE_SIGNAL, &Action, nullptr);
	}

	return RetVal;
}

uint32 FUnixPlatformProcess::GetCurrentProcessId()
{
	return getpid();
}

uint32 FUnixPlatformProcess::GetCurrentCoreNumber()
{
	return sched_getcpu();
}

void FUnixPlatformProcess::SetCurrentWorkingDirectoryToBaseDir()
{
#if defined(DISABLE_CWD_CHANGES) && DISABLE_CWD_CHANGES != 0
	check(false);
#else
	FPlatformMisc::CacheLaunchDir();
	chdir(TCHAR_TO_ANSI(FPlatformProcess::BaseDir()));
#endif
}

FString FUnixPlatformProcess::GetCurrentWorkingDirectory()
{
	// get the current directory
	ANSICHAR CurrentDir[UNIX_MAX_PATH] = { 0 };
	(void)getcwd(CurrentDir, sizeof(CurrentDir));
	return UTF8_TO_TCHAR(CurrentDir);
}

bool FUnixPlatformProcess::GetPerFrameProcessorUsage(uint32 ProcessId, float& ProcessUsageFraction, float& IdleUsageFraction)
{
	static bool bProcStatPrintHasHappened = false;
	static bool bProcStatGetLinePrintHappened = false;
	static bool bProcStatParsePrintHappened = false;
	static bool bProcPIDPrintHappened = false;
	static bool bProcPIDParsePrintHappened = false;
	static uint64 PreviousTotalTicks = 0;
	static uint64 PreviousUserTimeTicks = 0;
	static uint64 PreviousSystemTimeTicks = 0;
	static uint64 PreviousChildUserTimeTicks = 0;
	static uint64 PreviousChildSystemTimeTicks = 0;

	uint64_t UserTime, NiceTime, SystemTime, SoftIRQTime, IRQTime, IdleTime, IOWaitTime;
	char CPUName[8];
	char PIDString[20];
	char Buffer[500];
	
	FILE* ProcStatFile = fopen("/proc/stat", "r");

	if (ProcStatFile == nullptr)
	{
		UE_CLOGF(!bProcStatPrintHasHappened, LogUtilization, Verbose, "Failed to open /proc/stat.");
		bProcStatPrintHasHappened = true;
		return false;
	}

	// First line of the file is the total CPU usage so no need to tally everything together
	if (fgets(Buffer, sizeof(Buffer), ProcStatFile) == nullptr)
	{
		UE_CLOGF(!bProcStatParsePrintHappened, LogUtilization, Verbose, "Failed to get the first line of /proc/stat.");
		bProcStatParsePrintHappened = true;
		fclose(ProcStatFile);
		return false;
	}

	fclose(ProcStatFile);

	int Result = sscanf(Buffer, "%5s %8lu %8lu %8lu %8lu %8lu %8lu %8lu", CPUName,
		&UserTime, &NiceTime, &SystemTime, &IdleTime, &IOWaitTime, &IRQTime, &SoftIRQTime);
	if (Result != 8)
	{
		UE_CLOGF(!bProcStatGetLinePrintHappened, LogUtilization, Verbose, "Failed to parse the first line of /proc/stat.");
		bProcStatGetLinePrintHappened = true;
		return false;
	}

	uint64 TotalTicks = UserTime + NiceTime + SystemTime + SoftIRQTime + IRQTime + IdleTime + IOWaitTime;

	sprintf(PIDString, "/proc/%d/stat", ProcessId);
	FILE * ProcPIDStatFile = fopen(PIDString, "r");

	if (ProcPIDStatFile == nullptr)
	{
		UE_CLOGF(!bProcPIDPrintHappened, LogUtilization, Verbose, "Could not open /proc/%d/stat to read PID usage.", ProcessId);
		bProcPIDPrintHappened = true;
		return false;
	}
	
	if (fgets(Buffer, sizeof(Buffer), ProcPIDStatFile) == nullptr)
	{
		UE_CLOGF(!bProcPIDParsePrintHappened, LogUtilization, Verbose, "Could not get the first line of the PID stats file.");
		bProcPIDParsePrintHappened = true;
		fclose(ProcPIDStatFile);
		return false;
	}
	
	fclose(ProcPIDStatFile);

	// We only need the fields around the total time spent in user and system for both this process and its children
	// These are from man proc stat
	constexpr uint8 UserTimeField = 14;
	constexpr uint8 SystemTimeField = 15;
	constexpr uint8 ChildUserTimeField = 16;
	constexpr uint8 ChildSystemTimeField = 17;
	uint64 UserTimeTicks = 0;
	uint64 SystemTimeTicks = 0;
	uint64 ChildUserTimeTicks = 0;
	uint64 ChildSystemTimeTicks = 0;

	char* Remainder = nullptr;
	char* Token =  strtok_r(Buffer, " ", &Remainder);
	uint32 NumTokens = 1;

	while (Token != nullptr)
	{
		if (NumTokens == UserTimeField)
		{
			UserTimeTicks = atol(Token);
		}
		if (NumTokens == SystemTimeField)
		{
			SystemTimeTicks = atol(Token);
		}
		if (NumTokens == ChildUserTimeField)
		{
			ChildUserTimeTicks = atol(Token);
		}
		if (NumTokens == ChildSystemTimeField)
		{
			ChildSystemTimeTicks = atol(Token);
			break;
		}

		Token = strtok_r(nullptr, " ", &Remainder);
		NumTokens++;
	}

	// First sample period, there will be no valid data here
	// TODO: Catch a large delta 
	if (PreviousTotalTicks == 0)
	{
		PreviousTotalTicks = TotalTicks;
		PreviousUserTimeTicks = UserTimeTicks;
		PreviousSystemTimeTicks = SystemTimeTicks;
		PreviousChildUserTimeTicks = ChildUserTimeTicks;
		PreviousChildSystemTimeTicks = ChildSystemTimeTicks;
		return false;	
	}

	// Quick math
	uint32 TotalTicksDelta = TotalTicks - PreviousTotalTicks;
	uint32 UserTimeDelta = UserTimeTicks - PreviousUserTimeTicks;
	uint32 SystemTimeDelta = SystemTimeTicks - PreviousSystemTimeTicks;
	uint32 ChildUserTimeDelta = ChildUserTimeTicks - PreviousChildUserTimeTicks;
	uint32 ChildSystemTimeDelta = ChildSystemTimeTicks - PreviousChildSystemTimeTicks;

	uint32 TotalProcessTicksDelta = UserTimeDelta + SystemTimeDelta + ChildUserTimeDelta + ChildSystemTimeDelta;
	if (TotalTicksDelta == 0 || TotalProcessTicksDelta > TotalTicksDelta)
	{
		return false;
	}
	
	ProcessUsageFraction = (float)TotalProcessTicksDelta/(float)TotalTicksDelta * 100.0f;
	// Not sure what the correct logic would be here because we do not have an idle time for this process
	IdleUsageFraction = 100.0f - ProcessUsageFraction;

	PreviousTotalTicks = TotalTicks;
	PreviousUserTimeTicks = UserTimeTicks;
	PreviousSystemTimeTicks = SystemTimeTicks;
	PreviousChildUserTimeTicks = ChildUserTimeTicks;
	PreviousChildSystemTimeTicks = ChildSystemTimeTicks;
	
 	return true;
}

bool FUnixPlatformProcess::GetProcReturnCode(FProcHandle& ProcHandle, int32* ReturnCode)
{
	if (IsProcRunning(ProcHandle))
	{
		return false;
	}

	FProcState * ProcInfo = ProcHandle.GetProcessInfo();
	if (ProcInfo)
	{
		return ProcInfo->GetReturnCode(ReturnCode);
	}
	else if (ProcHandle.Get() != -1)
	{
		// Linux has no API to retrieve the exit code of a non-child process
		UE_LOGF(LogHAL, Warning, "GetProcReturnCode: cannot retrieve exit code for OpenProcess() handle (pid %d) -- not the parent process.",
			static_cast<int32>(ProcHandle.Get()));
	}

	return false;
}

bool FUnixPlatformProcess::Daemonize()
{
	if (daemon(1, 1) == -1)
	{
		int ErrNo = errno;
		UE_LOGF(LogHAL, Warning, "daemon(1, 1) failed with errno = %d (%ls)", ErrNo,
			StringCast< TCHAR >(strerror(ErrNo)).Get());
		return false;
	}

	return true;
}

bool FUnixPlatformProcess::IsApplicationRunning( uint32 ProcessId )
{
	// PID 0 is not a valid user application so lets ignore it as valid
	if (ProcessId == 0)
	{
		return false;
	}

	errno = 0;
	getpriority(PRIO_PROCESS, ProcessId);
	return errno == 0;
}

bool FUnixPlatformProcess::IsApplicationRunning( const TCHAR* ProcName )
{
	FString Commandline = "pidof '";
	Commandline += ProcName;
	Commandline += TEXT("'  > /dev/null");
	return !system(TCHAR_TO_UTF8(*Commandline));
}

static bool ReadPipeToStr(void *PipeRead, FString *OutStr)
{
	if (PipeRead)
	{
		FString NewLine = FPlatformProcess::ReadPipe(PipeRead);

		if (NewLine.Len() > 0)
		{
			if (OutStr != nullptr)
			{
				*OutStr += NewLine;
			}

			return true;
		}
	}

	return false;
}

bool FUnixPlatformProcess::ExecProcess(const TCHAR* URL, const TCHAR* Params, int32* OutReturnCode, FString* OutStdOut, FString* OutStdErr, const TCHAR* OptionalWorkingDirectory, bool bShouldEndWithParentProcess)
{
	FString CmdLineParams = Params;
	FString ExecutableFileName = URL;
	int32 ReturnCode = -1;

	void* PipeReadStdOut = nullptr;
	void* PipeWriteStdOut = nullptr;
	verify(FPlatformProcess::CreatePipe(PipeReadStdOut, PipeWriteStdOut));

	void* PipeReadStdErr = nullptr;
	void* PipeWriteStdErr = nullptr;
	if (OutStdErr)
	{
		verify(FPlatformProcess::CreatePipe(PipeReadStdErr, PipeWriteStdErr));
	}

	bool bInvoked = false;

	const bool bLaunchDetached = true;
	const bool bLaunchHidden = false;
	const bool bLaunchReallyHidden = bLaunchHidden;

	FProcHandle ProcHandle = FPlatformProcess::CreateProc(*ExecutableFileName, *CmdLineParams, bLaunchDetached, bLaunchHidden, bLaunchReallyHidden, NULL, 0, OptionalWorkingDirectory, PipeWriteStdOut, nullptr, PipeWriteStdErr);

	if (ProcHandle.IsValid())
	{
		while (FPlatformProcess::IsProcRunning(ProcHandle))
		{
			ReadPipeToStr(PipeReadStdOut, OutStdOut);
			ReadPipeToStr(PipeReadStdErr, OutStdErr);
			FPlatformProcess::Sleep(0.0);
		}

		// Read the remainder
		bool bReadingStdOut = true;
		bool bReadingStdErr = true;
		while (bReadingStdOut || bReadingStdErr)
		{
			if (bReadingStdOut && !ReadPipeToStr(PipeReadStdOut, OutStdOut))
			{
				bReadingStdOut = false;
			}

			if (bReadingStdErr && !ReadPipeToStr(PipeReadStdErr, OutStdErr))
			{
				bReadingStdErr = false;
			}
		}

		bInvoked = true;
		bool bGotReturnCode = FPlatformProcess::GetProcReturnCode(ProcHandle, &ReturnCode);
		check(bGotReturnCode)
		if (OutReturnCode != nullptr)
		{
			*OutReturnCode = ReturnCode;
		}

		FPlatformProcess::CloseProc(ProcHandle);
	}
	else
	{
		bInvoked = false;
		if (OutReturnCode != nullptr)
		{
			*OutReturnCode = -1;
		}
		if (OutStdOut != nullptr)
		{
			*OutStdOut = "";
		}
		if (OutStdErr != nullptr)
		{
			*OutStdErr = "";
		}
		UE_LOGF(LogHAL, Warning, "Failed to launch Tool. (%ls)", *ExecutableFileName);
	}

	FPlatformProcess::ClosePipe(PipeReadStdOut, PipeWriteStdOut);
	FPlatformProcess::ClosePipe(PipeReadStdErr, PipeWriteStdErr);
	return bInvoked;
}

bool FUnixPlatformProcess::LaunchFileInDefaultExternalApplication( const TCHAR* FileName, const TCHAR* Parms, ELaunchVerb::Type Verb, bool bPromptToOpenOnFailure)
{
	// TODO This ignores parms and verb
	pid_t pid = fork();
	if (pid == 0)
	{
		exit(execl("/usr/bin/xdg-open", "xdg-open", TCHAR_TO_UTF8(FileName), (char *)0));
	}

	return pid != -1;
}

void FUnixPlatformProcess::ExploreFolder( const TCHAR* FilePath )
{
	struct stat st;
	TCHAR TruncatedPath[UNIX_MAX_PATH] = TEXT("");
	FCString::Strcpy(TruncatedPath, FilePath);

	if (stat(TCHAR_TO_UTF8(FilePath), &st) == 0)
	{
		// we just want the directory portion of the path
		if (!S_ISDIR(st.st_mode))
		{
			for (int i=FCString::Strlen(TruncatedPath)-1; i > 0; i--)
			{
				if (TruncatedPath[i] == TCHAR('/'))
				{
					TruncatedPath[i] = 0;
					break;
				}
			}
		}

		// launch file manager
		pid_t pid = fork();
		if (pid == 0)
		{
			exit(execl("/usr/bin/xdg-open", "xdg-open", TCHAR_TO_UTF8(TruncatedPath), (char *)0));
		}
	}
}

/**
 * Private struct to store implementation specific data.
 */
struct FProcEnumData
{
	// Array of processes.
	TArray<FUnixPlatformProcess::FProcEnumInfo> Processes;

	// Current process id.
	uint32 CurrentProcIndex;
};

FUnixPlatformProcess::FProcEnumerator::FProcEnumerator()
{
		Data = new FProcEnumData;
	Data->CurrentProcIndex = -1;
	
	TArray<uint32> PIDs;
	
	class FPIDsCollector : public IPlatformFile::FDirectoryVisitor
	{
	public:
		FPIDsCollector(TArray<uint32>& InPIDsToCollect)
			: PIDsToCollect(InPIDsToCollect)
		{ }

		bool Visit(const TCHAR* FilenameOrDirectory, bool bIsDirectory)
		{
			FString StrPID = FPaths::GetBaseFilename(FilenameOrDirectory);
			
			if (bIsDirectory && FCString::IsNumeric(*StrPID))
			{
				PIDsToCollect.Add(FCString::Atoi(*StrPID));
			}
			
			return true;
		}

	private:
		TArray<uint32>& PIDsToCollect;
	} PIDsCollector(PIDs);

	IPlatformFile::GetPlatformPhysical().IterateDirectory(TEXT("/proc"), PIDsCollector);
	
	for (auto PID : PIDs)
	{
		Data->Processes.Add(FProcEnumInfo(PID));
	}
}

FUnixPlatformProcess::FProcEnumerator::~FProcEnumerator()
{
	delete Data;
}

FUnixPlatformProcess::FProcEnumInfo FUnixPlatformProcess::FProcEnumerator::GetCurrent() const
{
	return Data->Processes[Data->CurrentProcIndex];
}

bool FUnixPlatformProcess::FProcEnumerator::MoveNext()
{
	if (Data->CurrentProcIndex + 1 == Data->Processes.Num())
	{
		return false;
	}

	++Data->CurrentProcIndex;

	return true;
}

FUnixPlatformProcess::FProcEnumInfo::FProcEnumInfo(uint32 InPID)
	: PID(InPID)
{

}

uint32 FUnixPlatformProcess::FProcEnumInfo::GetPID() const
{
	return PID;
}

uint32 FUnixPlatformProcess::FProcEnumInfo::GetParentPID() const
{
	char Buf[256];
	uint32 DummyNumber;
	char DummyChar;
	uint32 ParentPID;
	
	sprintf(Buf, "/proc/%d/stat", GetPID());
	
	FILE* FilePtr = fopen(Buf, "r");
	if (FilePtr == nullptr)
	{
		return 1;
	}
	if (fscanf(FilePtr, "%d %s %c %d", &DummyNumber, Buf, &DummyChar, &ParentPID) != 4)
	{
		ParentPID = 1;
	}
	fclose(FilePtr);

	return ParentPID;
}

FString FUnixPlatformProcess::FProcEnumInfo::GetFullPath() const
{
	return GetApplicationName(GetPID());
}

FString FUnixPlatformProcess::FProcEnumInfo::GetName() const
{
	return FPaths::GetCleanFilename(GetFullPath());
}

void FUnixPlatformProcess::OnChildEndFramePostFork()
{
	FCoreDelegates::OnEndFrame.Remove(OnEndFrameHandle);
	OnEndFrameHandle.Reset();

	FCoreDelegates::OnChildEndFramePostFork.Broadcast();
}

int32 FUnixPlatformProcess::TranslateThreadPriority(EThreadPriority Priority)
{
	// In general, the range is -20 to 19 (negative is highest, positive is lowest)
	int32 NiceLevel = 0;
	switch (Priority)
	{
		case TPri_TimeCritical:
			NiceLevel = -20;
			break;

		case TPri_Highest:
			NiceLevel = -15;
			break;

		case TPri_AboveNormal:
			NiceLevel = -10;
			break;

		case TPri_Normal:
			NiceLevel = 0;
			break;

		case TPri_SlightlyBelowNormal:
			NiceLevel = 3;
			break;

		case TPri_BelowNormal:
			NiceLevel = 5;
			break;

		case TPri_Lowest:
			NiceLevel = 10;		// 19 is a total starvation
			break;

		default:
			UE_LOGF(LogHAL, Fatal, "Unknown Priority passed to FRunnableThreadPThread::TranslateThreadPriority()");
			return 0;
	}

	// note: a non-privileged process can only go as low as RLIMIT_NICE
	return NiceLevel;
}

void FUnixPlatformProcess::SetThreadNiceValue(uint32_t ThreadId, int32 NiceValue)
{
	// We still try to set priority, but failure is not considered as an error
	if (setpriority(PRIO_PROCESS, ThreadId, NiceValue) != 0 && WITH_PROCESS_PRIORITY_CONTROL)
	{
		static bool bIsLogged = false;
		if (!bIsLogged)
		{
			bIsLogged = true;
			// Unfortunately this is going to be a frequent occurence given that by default Unix doesn't allow raising priorities.
			// NOTE: In WSL run "sudo prlimit --nice=40 --pid $$" to promote current shell to change nice values.
			int ErrNo = errno;
			UE_LOGF(LogHAL, Error, "Can't set nice to %d. Reason = %ls. Do you have CAP_SYS_NICE capability?", NiceValue, ANSI_TO_TCHAR(strerror(ErrNo)));
		}
	}
}

void FUnixPlatformProcess::SetThreadPriority(EThreadPriority NewPriority)
{
	uint32 ThreadId = FPlatformTLS::GetCurrentThreadId();
	int32 NiceValue = FUnixPlatformProcess::TranslateThreadPriority(NewPriority);

	FUnixPlatformProcess::SetThreadNiceValue(ThreadId, NiceValue);
}
