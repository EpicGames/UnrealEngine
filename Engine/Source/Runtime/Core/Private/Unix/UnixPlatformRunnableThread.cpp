// Copyright Epic Games, Inc. All Rights Reserved.

#include "Unix/UnixPlatformRunnableThread.h"
#include "Unix/UnixPlatformProcess.h"
#include "HAL/UnrealMemory.h"
#include "Templates/UnrealTemplate.h"
#include "Containers/UnrealString.h"
#include "Containers/StringConv.h"
#include "Logging/LogMacros.h"
#include "Misc/AssertionMacros.h"

#include <sys/resource.h>

bool FRunnableThreadUnix::SetupSignalHandlerStack(void* StackBuffer, const size_t StackBufferSize, void** OutStackGuardPageAddress)
{
	if (!StackBuffer)
	{
		return false;
	}

	// Added by CL 11188846 for ASan, TSan, and UBSan
	// #jira UE-62784 UE-62803 UE-62804
	if (FORCE_ANSI_ALLOCATOR)
	{
		return true;
	}

	// find an address close to begin of the stack and protect it
	uint64 StackGuardPage = reinterpret_cast<uint64>(StackBuffer);

	// align by page
	const uint64 PageSize = sysconf(_SC_PAGESIZE);
	const uint64 Remainder = StackGuardPage % PageSize;
	if (Remainder != 0)
	{
		StackGuardPage += (PageSize - Remainder);
		checkf(StackGuardPage % PageSize == 0, TEXT("StackGuardPage is not aligned on page size"));
	}

	checkf(StackGuardPage + PageSize - reinterpret_cast<uint64>(StackBuffer) < StackBufferSize,
		TEXT("Stack size is too small for the extra guard page!"));

	void* StackGuardPageAddr = reinterpret_cast<void*>(StackGuardPage);
	if (FPlatformMemory::PageProtect(StackGuardPageAddr, PageSize, true, false))
	{
		if (OutStackGuardPageAddress)
		{
			*OutStackGuardPageAddress = StackGuardPageAddr;
		}
	}
	else
	{
		// cannot use UE_LOG - can run into deadlocks in output device code
		fprintf(stderr, "Unable to set a guard page on the alt stack\n");
	}

	// set up the buffer to be used as stack
	stack_t SignalHandlerStack;
	FMemory::Memzero(SignalHandlerStack);
	SignalHandlerStack.ss_sp = StackBuffer;
	SignalHandlerStack.ss_size = StackBufferSize;

	bool bSuccess = (sigaltstack(&SignalHandlerStack, nullptr) == 0);
	if (!bSuccess)
	{
		int ErrNo = errno;
		// cannot use UE_LOG - can run into deadlocks in output device code
		fprintf(stderr, "Unable to set alternate stack for crash handler, sigaltstack() failed with errno=%d (%s)\n", ErrNo, strerror(ErrNo));
	}

	return bSuccess;
}

int32 FRunnableThreadUnix::TranslateThreadPriority(EThreadPriority Priority)
{
	return FUnixPlatformProcess::TranslateThreadPriority(Priority);
}

void FRunnableThreadUnix::SetThreadPriority(EThreadPriority NewPriority)
{
	// always set priority to avoid second guessing
	ThreadPriority = NewPriority;
	SetThreadPriority(Thread, NewPriority);
}

void FRunnableThreadUnix::SetThreadPriority(pthread_t InThread, EThreadPriority NewPriority)
{
	// NOTE: InThread is ignored, but we can use ThreadID that maps to SYS_ttid
	int32 Prio = TranslateThreadPriority(NewPriority);

	// Unix implements thread priorities the same way as process priorities, while on Windows they are relative to process priority.
	// We want Windows behavior, since sometimes we set the whole process to a lower priority and would like its threads to avoid raising it
	// (even if RTLIMIT_NICE allows it) - example is ShaderCompileWorker that need to run in the background.
	//
	// Thusly we remember the baseline value that the process has at the moment of first priority change and set thread priority relative to it.
	// This is of course subject to race conditions and other problems (e.g. in case main thread changes its priority after the fact), but it's the best we have.

	if (!bGotBaselineNiceValue)
	{
		// checking errno is necessary since -1 is a valid priority to return from getpriority()
		errno = 0;
		int CurrentPriority = getpriority(PRIO_PROCESS, getpid());
		// if getting priority wasn't successful, don't change the baseline value (will be 0 - i.e. normal - by default)
		if (CurrentPriority != -1 || errno == 0)
		{
			BaselineNiceValue = CurrentPriority;
			bGotBaselineNiceValue = true;
		}
	}

	int ModifiedPriority = FMath::Clamp(BaselineNiceValue + Prio, -20, 19);

	FUnixPlatformProcess::SetThreadNiceValue(ThreadID, ModifiedPriority);
}

void FRunnableThreadUnix::PreRun()
{
	FString SizeLimitedThreadName = ThreadName;

	if (SizeLimitedThreadName.Len() > EConstants::UnixThreadNameLimit)
	{
		// first, attempt to cut out common and meaningless substrings
		SizeLimitedThreadName = SizeLimitedThreadName.Replace(TEXT("Thread"), TEXT(""));
		SizeLimitedThreadName = SizeLimitedThreadName.Replace(TEXT("Runnable"), TEXT(""));

		// if still larger
		if (SizeLimitedThreadName.Len() > EConstants::UnixThreadNameLimit)
		{
			FString Temp = SizeLimitedThreadName;

			// cut out the middle and replace with a substitute
			const TCHAR Dash[] = TEXT("-");
			const int32 DashLen = UE_ARRAY_COUNT(Dash) - 1;
			int NumToLeave = (EConstants::UnixThreadNameLimit - DashLen) / 2;

			SizeLimitedThreadName = Temp.Left(EConstants::UnixThreadNameLimit - (NumToLeave + DashLen));
			SizeLimitedThreadName += Dash;
			SizeLimitedThreadName += Temp.Right(NumToLeave);

			check(SizeLimitedThreadName.Len() <= EConstants::UnixThreadNameLimit);
		}
	}

	int ErrCode = pthread_setname_np(Thread, TCHAR_TO_ANSI(*SizeLimitedThreadName));
	if (ErrCode != 0)
	{
		UE_LOGF(LogHAL, Warning, "pthread_setname_np(, '%ls') failed with error %d (%ls).", *ThreadName, ErrCode, ANSI_TO_TCHAR(strerror(ErrCode)));
	}

	// set the alternate stack for handling crashes due to stack overflow
	check(ThreadCrashHandlingStack == nullptr);
	ThreadCrashHandlingStack = AllocCrashHandlerStack();
	SetupSignalHandlerStack(ThreadCrashHandlingStack, FRunnableThreadUnix::GetCrashHandlerStackSize(), &StackGuardPageAddress);
}

void FRunnableThreadUnix::PostRun()
{
	if (StackGuardPageAddress != nullptr)
	{
		// we protected one page only
		const uint64 PageSize = sysconf(_SC_PAGESIZE);

		if (!FPlatformMemory::PageProtect(StackGuardPageAddress, PageSize, true, true))
		{
			UE_LOGF(LogCore, Error, "Unable to remove a guard page from the alt stack");
		}

		StackGuardPageAddress = nullptr;
	}

	if (ThreadCrashHandlingStack != nullptr)
	{
		FreeCrashHandlerStack(ThreadCrashHandlingStack);
		ThreadCrashHandlingStack = nullptr;
	}
}

uint32 FRunnableThreadUnix::AdjustStackSize(uint32 InStackSize)
{
	InStackSize = FRunnableThreadPThread::AdjustStackSize(InStackSize);

	// If it's set, make sure it's at least 128 KB or stack allocations (e.g. in Logf) may fail
	if (InStackSize && InStackSize < 128 * 1024)
	{
		InStackSize = 128 * 1024;
	}

	return InStackSize;
}
