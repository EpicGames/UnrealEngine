// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	UnixPlatformStackWalk.cpp: Unix implementations of stack walk functions
=============================================================================*/

#include "Unix/UnixPlatformStackWalk.h"
#include "CoreGlobals.h"
#include "Misc/Paths.h"
#include "Misc/Parse.h"
#include "Misc/ScopeLock.h"
#include "Misc/CommandLine.h"
#include "Unix/UnixPlatformCrashContext.h"
#include "Unix/UnixPlatformRealTimeSignals.h" 
#include "HAL/ExceptionHandling.h"
#include "HAL/PlatformProcess.h"
#include "HAL/PlatformTime.h"
#include "Templates/UnrealTemplate.h"
#include "AutoRTFM.h"

#include <atomic>
#include <link.h>
#include <signal.h>

#include "HAL/IConsoleManager.h"

#include <sys/mman.h>

static TAutoConsoleVariable<float> CVarUnixPlatformThreadCallStackMaxWait(
	TEXT("UnixPlatformThreadStackWalk.MaxWait"),
	60.0f,
	TEXT("The number of seconds allowed to spin before killing the process, with the assumption the signal handler has hung."));

// When true, thread stack capture timeout should not cause a deliberate crash.
// Set during crash handling to prevent recursive crashes when gathering all thread stacks.
bool GUnixStackWalkNoCrashOnTimeout = false;

// Set while a stack walk is in progress on the current thread. Without this guard,
// a failure inside the stack walker can recurse until the stack is exhausted.
static thread_local bool GStackWalkInProgress = false;

// Returns true if it is safe to start a stack walk on the current thread. Returns false
// if one is already in progress (the caller should bail out); in that case, an optional
// caller-supplied buffer is populated with a short marker so its output path is not
// left with stale data.
static bool CanProcessStackWalk(ANSICHAR* HumanReadableString = nullptr, SIZE_T HumanReadableStringSize = 0)
{
	if (GStackWalkInProgress)
	{
		if (HumanReadableString && HumanReadableStringSize > 0)
		{
			FCStringAnsi::Strncpy(HumanReadableString, "[stack walker re-entered; skipping]\n", HumanReadableStringSize);
		}
		return false;
	}
	return true;
}

// Init'ed in UnixPlatformMemory. Once this is tested more we can remove this fallback flag
extern bool CORE_API GFullCrashCallstack;

// Init'ed in UnixPlatformMemory.
extern bool CORE_API GTimeEnsures;

namespace
{
	// If we want to load into memory the modules symbol file, it will be allocated to this pointer
	uint8_t* GModuleSymbolFileMemory = nullptr;
	size_t   GModuleSymbolFileMemorySize = 0U;
}

void CORE_API UnixPlatformStackWalk_UnloadPreloadedModuleSymbol()
{
	if (GModuleSymbolFileMemory)
	{
		GModuleSymbolFileMemory -= FPlatformMemory::GetConstants().PageSize;
		FMemory::Free(GModuleSymbolFileMemory);
		GModuleSymbolFileMemory = nullptr;
	}
}

void CORE_API UnixPlatformStackWalk_PreloadModuleSymbolFile()
{
	if (GModuleSymbolFileMemory == nullptr)
	{
		FString ModuleSymbolPath = FUnixPlatformProcess::GetApplicationName(getpid()) + ".sym";
		int SymbolFileFD = open(TCHAR_TO_UTF8(*ModuleSymbolPath), O_RDONLY);

		if (SymbolFileFD == -1)
		{
			int ErrNo = errno;
			UE_LOGF(LogHAL, Warning, "UnixPlatformStackWalk_PreloadedModuleSymbol: open() failed on path %ls errno=%d (%ls)",
				*ModuleSymbolPath,
				ErrNo,
				UTF8_TO_TCHAR(strerror(ErrNo)));
		}
		else
		{
			lseek(SymbolFileFD, 0, SEEK_END);
			GModuleSymbolFileMemorySize = lseek(SymbolFileFD, 0, SEEK_CUR);
			lseek(SymbolFileFD, 0, SEEK_SET);

			// Allocate and jump by an extra page size so we can make sure we read only *our* memory and dont read only someone elses.
			GModuleSymbolFileMemory = (uint8_t*)FMemory::Malloc(GModuleSymbolFileMemorySize +  2 * (FPlatformMemory::GetConstants().PageSize));
			UE_LOGF(LogHAL, Warning, "UnixPlatformStackWalk_PreloadModuleSymbolFile: GModuleSymbolFileMemory = 0x%llx, GModuleSymbolFileMemorySize = %zu bytes", GModuleSymbolFileMemory, GModuleSymbolFileMemorySize);

			GModuleSymbolFileMemory += FPlatformMemory::GetConstants().PageSize;
			UE_LOGF(LogHAL, Warning, "UnixPlatformStackWalk_PreloadModuleSymbolFile: GModuleSymbolFileMemory = 0x%llx (After adding additional memory page", GModuleSymbolFileMemory);

			// On linux read() will transfer at most 2,147,479,552 bytes
			const int32 MaxBytesToRead = 0x7FFFF000;

			ssize_t BytesRead = 0;
			{
				// RemainingBytes must be size_t to prevent wrap around if GModuleSymbolFileMemorySize is too large
				size_t RemainingBytes = GModuleSymbolFileMemorySize;
				uint8_t* CurrentModulePos = GModuleSymbolFileMemory;
				ssize_t CurrentBytesRead = 0;

				while(RemainingBytes > MaxBytesToRead)
				{
					CurrentBytesRead = read(SymbolFileFD, CurrentModulePos, MaxBytesToRead);
					if(CurrentBytesRead < 0)
					{
						break;
					}
					RemainingBytes -= CurrentBytesRead;
					CurrentModulePos += CurrentBytesRead;
					BytesRead += CurrentBytesRead;
				}
				BytesRead += read(SymbolFileFD, CurrentModulePos, RemainingBytes);
			}


			close(SymbolFileFD);

			// Did not read expected amount of bytes
			if (BytesRead != GModuleSymbolFileMemorySize)
			{
				UE_LOGF(LogHAL, Warning, "UnixPlatformStackWalk_PreloadedModuleSymbol: BytesRead %zd Expected %zu", BytesRead, GModuleSymbolFileMemorySize);
				UnixPlatformStackWalk_UnloadPreloadedModuleSymbol();

				if (BytesRead == -1)
				{
					int ErrNo = errno;
					UE_LOGF(LogHAL, Warning, "UnixPlatformStackWalk_PreloadedModuleSymbol: read() failed, errno=%d (%ls)",
						ErrNo,
						UTF8_TO_TCHAR(strerror(ErrNo)));
				}
			}
			else
			{
				// Mark our selfs to the left most page boundary read only, we allocated and moved down our memory by a page to give us some slack. Only do this if we've not freed GModuleSymbolFileMemory!
				mprotect(reinterpret_cast<void*>(reinterpret_cast<uint64>(GModuleSymbolFileMemory) & ~(FPlatformMemory::GetConstants().PageSize - 1)), GModuleSymbolFileMemorySize, PROT_READ);
			}
		}
	}
}

namespace
{
	// Only used for testing ensure timing
	bool GHandlingEnsure = false;

	// These structures are copied from Engine/Source/Programs/BreakpadSymbolEncoder/BreakpadSymbolEncoder.h
	// DO NOT CHANGE THE SIZE OF THESES STRUCTURES (Unless the BreakpadSymbolEncoder.h structures have changed)
#pragma pack(push, 1)
	struct RecordsHeader
	{
		uint32_t RecordCount;
	};

	struct Record
	{
		uint64_t Address    = static_cast<uint64_t>(-1);
		uint32_t LineNumber = static_cast<uint32_t>(-1);
		uint32_t FileRelativeOffset   = static_cast<uint32_t>(-1);
		uint32_t SymbolRelativeOffset = static_cast<uint32_t>(-1);
	};
#pragma pack(pop)

	struct RecordReader
	{
		virtual ~RecordReader() = default;
		virtual bool IsValid() const = 0;
		virtual void Read(void* Buffer, uint32_t Size, uint32_t Offset) const = 0;
	};

	class MemoryReader : public RecordReader
	{
	public:
		void Init(const uint8_t* InRecordMemory, size_t InMemorySize)
		{
			RecordMemory = InRecordMemory;
			MemorySize   = InMemorySize;
		}

		bool IsValid() const override
		{
			return RecordMemory != nullptr;
		}

		void Read(void* Buffer, uint32_t Size, uint32_t Offset) const override
		{
			if (Offset >= MemorySize)
			{
				return;
			}

			uint32_t MaxSize = MemorySize - Offset;

			memcpy(Buffer, RecordMemory + Offset, FMath::Min(Size, MaxSize));
		}

	private:
		const uint8_t* RecordMemory = nullptr;
		size_t MemorySize = 0U;
	};

	class FDReader : public RecordReader
	{
	public:
		FDReader() = default;

		~FDReader()
		{
			if (SymbolFileFD != -1)
			{
				close(SymbolFileFD);
			}
		}

		void Init(const char* Path)
		{
			SymbolFileFD = open(Path, O_RDONLY);
		}

		bool IsValid() const override
		{
			return SymbolFileFD != -1;
		}

		void Read(void* Buffer, uint32_t Size, uint32_t Offset) const override
		{
			lseek(SymbolFileFD, Offset, SEEK_SET);
			read(SymbolFileFD, Buffer, Size);
		}

		// We own the FD, dont allow copying
		FDReader(const FDReader& Reader) = delete;
		FDReader& operator=(const FDReader& Reader) = delete;

	private:
		int SymbolFileFD = -1;
	};

	class SymbolFileReader
	{
	public:
		explicit SymbolFileReader(const RecordReader& InReader)
			: Reader(InReader)
			, StartOffset(sizeof(RecordsHeader))
		{
			if (Reader.IsValid())
			{
				Reader.Read(&RecordCount, sizeof(RecordCount), 0);
			}
		}

		bool IsValid() const
		{
			return Reader.IsValid() && RecordCount > 0;
		}

		uint32_t GetRecordCount() const
		{
			return RecordCount;
		}

		Record GetRecord(int Index) const
		{
			// When we remove this check, make sure we handle possible out of bounds cases
			if (Index > RecordCount || Index < 0)
			{
				return {};
			}

			Record Out;
			uint32_t RecordOffset = StartOffset + Index * sizeof(Record);
			Reader.Read(&Out, sizeof(Record), RecordOffset);

			return Out;
		}

		void ReadOffsetIntoMemory(char* Buffer, size_t MaxSize, uint32_t Offset)
		{
			// We need at least a single byte to record an empty string into `Buffer`
			check(MaxSize > 0);

			// Offset unsigned -1 (0xffffffff) is invalid
			if (LIKELY(Offset != static_cast<uint32_t>(-1)))
			{
				uint32_t StartOfStrings = StartOffset + RecordCount * sizeof(Record);
				Reader.Read(Buffer, MaxSize, StartOfStrings + Offset);

				// Read the max chunk we can read, then find the next '\n' and replace that with '\0'
				for (int i = 0; i < MaxSize; i++)
				{
					if (Buffer[i] == '\n')
					{
						Buffer[i] = '\0';
						return;
					}
				}
			}

			// We couldnt find the end of the line, lets assume we failed to read in a line
			Buffer[0] = '\0';
		}

	private:
		const RecordReader& Reader;

		// For now can only be up to 4GB
		uint32_t StartOffset = 0;
		uint32_t RecordCount = 0;
	};

	bool PopulateProgramCounterSymbolInfoFromSymbolFile(uint64 ProgramCounter, FProgramCounterSymbolInfo& out_SymbolInfo)
	{
		bool CheckingEnsureTime = GTimeEnsures && GHandlingEnsure;
		double StartTime = CheckingEnsureTime ? FPlatformTime::Seconds() : 0.0;

		double DladdrEndTime = StartTime;
		double RecordReaderEndTime = StartTime;
		double SearchEndTime = StartTime;

		bool RecordFound = false;

		Dl_info info;
		bool bDladdrRet = 0;

		bDladdrRet = dladdr(reinterpret_cast<void*>(ProgramCounter), &info);
		DladdrEndTime = CheckingEnsureTime ? FPlatformTime::Seconds() : 0.0;

		if (bDladdrRet != 0)
		{
			out_SymbolInfo.ProgramCounter = ProgramCounter;

			if (UNLIKELY(info.dli_fname == nullptr) ||
				UNLIKELY(info.dli_fbase == nullptr))
			{
				if (CheckingEnsureTime)
				{
					UE_LOGF(LogCore, VeryVerbose, "Symbol lookup time for 0x%016llx Dladdr: %lfms", reinterpret_cast<void*>(ProgramCounter), (DladdrEndTime - StartTime) * 1000);
				}

				// If we cannot find the module name or the module base return early
				return false;
			}

			const ANSICHAR* SOPath = info.dli_fname;
			const ANSICHAR* SOName = FCStringAnsi::Strrchr(SOPath, '/');
			if (SOName)
			{
				SOName += 1;
			}
			else
			{
				SOName = SOPath;
			}

			FCStringAnsi::Strcpy(out_SymbolInfo.ModuleName, SOName);
			out_SymbolInfo.OffsetInModule = ProgramCounter - reinterpret_cast<uint64>(info.dli_fbase);

			if (info.dli_saddr != nullptr)
			{
				out_SymbolInfo.SymbolDisplacement = ProgramCounter - reinterpret_cast<uint64>(info.dli_saddr);
			}
			// If we cant find the function either lets just give it the offset into the module
			else if (info.dli_sname == nullptr)
			{
				out_SymbolInfo.SymbolDisplacement = out_SymbolInfo.OffsetInModule;
			}

			if (info.dli_sname != nullptr)
			{
				FCStringAnsi::Strcpy(out_SymbolInfo.FunctionName, info.dli_sname);
			}

			ANSICHAR ModuleSymbolPath[UNIX_MAX_PATH + 1];

			// We cant assume if we are relative we have not chdir to a different working dir.
			if (FPaths::IsRelative(info.dli_fname))
			{
				FCStringAnsi::Strcpy(ModuleSymbolPath, TCHAR_TO_UTF8(FPlatformProcess::BaseDir()));
#if WITH_LOW_LEVEL_TESTS
				// Low level tests live one level above the base directory in a folder <ModuleName>Tests
				// Sometimes this folder can also be just <ModuleName> if the target was compiled with the tests
				// TODO: This code needs work as its only hardcoded to allows finding the *.sym for Development config.
				// Debug/Test/Shipping/ASan configs all fail here
				ANSICHAR ModuleDirectory[UNIX_MAX_PATH + 1];

				FCStringAnsi::Strcpy(ModuleDirectory, ModuleSymbolPath);
				FCStringAnsi::Strcat(ModuleDirectory, "/");
				FCStringAnsi::Strcat(ModuleDirectory, TCHAR_TO_UTF8(*FPaths::GetBaseFilename(out_SymbolInfo.ModuleName)));

				// use stat instead of FPaths::DirectoryExists as it calls into a static global which may be dead at exit time
				struct stat StatInfo;
				if (stat(ModuleDirectory, &StatInfo) == 0)
				{
					FCStringAnsi::Strcat(ModuleSymbolPath, "/");
					FCStringAnsi::Strcat(ModuleSymbolPath, TCHAR_TO_UTF8(*FPaths::GetBaseFilename(out_SymbolInfo.ModuleName)));
					FCStringAnsi::Strcat(ModuleSymbolPath, "/");
				}
				else
				{
					FCStringAnsi::Strcat(ModuleSymbolPath, "/");
					FCStringAnsi::Strcat(ModuleSymbolPath, TCHAR_TO_UTF8(*FPaths::GetBaseFilename(out_SymbolInfo.ModuleName)));
					FCStringAnsi::Strcat(ModuleSymbolPath, "Tests");
					FCStringAnsi::Strcat(ModuleSymbolPath, "/");
				}
#endif
				FCStringAnsi::Strcat(ModuleSymbolPath, TCHAR_TO_UTF8(*FPaths::GetBaseFilename(out_SymbolInfo.ModuleName)));
				FCStringAnsi::Strcat(ModuleSymbolPath, ".sym");
			}
			else
			{
				FCStringAnsi::Strcpy(ModuleSymbolPath, TCHAR_TO_UTF8(*FPaths::GetBaseFilename(info.dli_fname, false)));
				FCStringAnsi::Strcat(ModuleSymbolPath, ".sym");
			}

			RecordReader* RecordReader;
			FDReader ModuleFDReader;
			MemoryReader ModuleMemoryReader;

			// If we have preloaded our modules symbol file and the program counter we are trying to symbolicate is our main
			// module we can use this preloaded reader
			if (GModuleSymbolFileMemory && !FCStringAnsi::Strcmp(SOName, TCHAR_TO_UTF8(FPlatformProcess::ExecutableName())))
			{
				ModuleMemoryReader.Init(GModuleSymbolFileMemory, GModuleSymbolFileMemorySize);
				RecordReader = &ModuleMemoryReader;
			}
			else
			{
				// TODO We should look at only opening the file once per entire callstack (but it depends on the module names)
				ModuleFDReader.Init(ModuleSymbolPath);
				RecordReader = &ModuleFDReader;
			}

			SymbolFileReader Reader(*RecordReader);

			RecordReaderEndTime = CheckingEnsureTime ? FPlatformTime::Seconds() : 0.0;

			if (Reader.IsValid())
			{
				size_t Start = 0;
				size_t End   = Reader.GetRecordCount() - 1;
				bool bFound  = false;
				uint64 AddressToFind = out_SymbolInfo.OffsetInModule;

				// Make sure we arent not trying the same middle index over and over again
				size_t LastMiddle = 1;
				size_t Middle = 0;

				while (End - Start > 0 && LastMiddle != Middle)
				{
					LastMiddle = Middle;
					Middle     = (Start + End) / 2;

					if (Middle + 1 >= Reader.GetRecordCount())
					{
						// We have placed a dummy record at the end of all the records. So if our Middle index
						// happens to be that dummy record we are in a bad spot
						break;
					}

					Record Current = Reader.GetRecord(Middle);
					Record Next    = Reader.GetRecord(Middle + 1);
					size_t Size    = Next.Address - Current.Address;

					if (AddressToFind >= Current.Address && AddressToFind < Current.Address + Size)
					{
						// Hack when we have a zero line number to attempt to use the previous record for a better guess.
						// non-virtual thunks seem to cause a bunch of these but this will not fix those.
						if (Current.LineNumber == 0)
						{
							Record Previous = Reader.GetRecord(Middle - 1);
							if (Previous.LineNumber > 0 && Previous.LineNumber != static_cast<uint32_t>(-1))
							{
								Current.LineNumber = Previous.LineNumber;
							}
						}

						ANSICHAR TempFileName[sizeof(out_SymbolInfo.Filename)];
						Reader.ReadOffsetIntoMemory(TempFileName, sizeof(TempFileName), Current.FileRelativeOffset);
						FUnixPlatformStackWalk::CopyVirtualPathToLocal(out_SymbolInfo.Filename, sizeof(out_SymbolInfo.Filename), TempFileName);

						Reader.ReadOffsetIntoMemory(out_SymbolInfo.FunctionName, sizeof(out_SymbolInfo.FunctionName), Current.SymbolRelativeOffset);
						out_SymbolInfo.LineNumber = Current.LineNumber;

						// If we find a function but no sname from dladdr we cannot make assumptions about its symbol displacement.
						// And a Function name is better then the OffsetInModule address
						if (info.dli_sname == nullptr)
						{
							out_SymbolInfo.SymbolDisplacement = 0x0;
						}

						// If we dont have a file name we have to assume its just a public symbol and use the old way to demangle the backtrace info
						if (out_SymbolInfo.Filename[0] == '\0')
						{
							break;
						}

						RecordFound = true;
						break;
					}
					else if (AddressToFind > Current.Address)
					{
						Start = Middle;
					}
					else
					{
						End = Middle;
					}
				}
			}
			// We only care if we fail to find our own *.sym file
			else if (!FCStringAnsi::Strcmp(SOName, TCHAR_TO_UTF8(FPlatformProcess::ExecutableName())))
			{
				static bool bReported = false;
				if (!bReported)
				{
					// This will likely happen multiple times, so only write out once
					bReported = true;

					// Will not be part of UE_LOG as it would potentially allocate memory
					const ANSICHAR* Message = "Failed to find symbol file, expected location:\n\"";
					write(STDOUT_FILENO, Message, FCStringAnsi::Strlen(Message));
					write(STDOUT_FILENO, ModuleSymbolPath, FCStringAnsi::Strlen(ModuleSymbolPath));
					write(STDOUT_FILENO, "\"\n", 2);
				}
			}
		}

		SearchEndTime = CheckingEnsureTime ? FPlatformTime::Seconds() : 0.0;

		if (CheckingEnsureTime)
		{
			UE_LOGF(LogCore, VeryVerbose, "Symbol lookup time for 0x%016llx Dladdr: %lfms Open: %lfms Search: %lfms",
					ProgramCounter,
					(DladdrEndTime - StartTime) * 1000.0,
					(RecordReaderEndTime - DladdrEndTime) * 1000.0,
					(SearchEndTime - RecordReaderEndTime) * 1000.0);
		}

		return RecordFound;
	}
}

UE_AUTORTFM_ALWAYS_OPEN void FUnixPlatformStackWalk::ProgramCounterToSymbolInfo( uint64 ProgramCounter, FProgramCounterSymbolInfo& out_SymbolInfo )
{
	PopulateProgramCounterSymbolInfoFromSymbolFile(ProgramCounter, out_SymbolInfo);
}

bool FUnixPlatformStackWalk::ProgramCounterToHumanReadableString( int32 CurrentCallDepth, uint64 ProgramCounter, ANSICHAR* HumanReadableString, SIZE_T HumanReadableStringSize, FGenericCrashContext* Context )
{
	//
	// Callstack lines should be written in this standard format
	//
	//	0xaddress module!func [file]
	//
	// E.g. 0x045C8D01 OrionClient.self!UEngine::PerformError() [D:\Epic\Orion\Engine\Source\Runtime\Engine\Private\UnrealEngine.cpp:6481]
	//
	// Module may be omitted, everything else should be present, or substituted with a string that conforms to the expected type
	//
	// E.g 0x00000000 UnknownFunction []
	//
	//

	if (HumanReadableString && HumanReadableStringSize > 0)
	{
		ANSICHAR TempArray[MAX_SPRINTF];
		if (CurrentCallDepth < 0)
		{
			FCStringAnsi::Sprintf(TempArray, "0x%016llx ", ProgramCounter);
			FCStringAnsi::StrncatTruncateDest(HumanReadableString, HumanReadableStringSize, TempArray);

			// won't be able to display names here
		}
		else
		{
			FCStringAnsi::Sprintf(TempArray, "0x%016llx ", ProgramCounter);
			FCStringAnsi::StrncatTruncateDest(HumanReadableString, HumanReadableStringSize, TempArray);

			// Get filename, source file and line number
			FUnixCrashContext* UnixContext = static_cast< FUnixCrashContext* >( Context );

			// do not even attempt to get detailed info for continuable events (like ensure) as it will result in long hitch, use the fast path
			bool bAddDetailedInfo = false;
			if (UnixContext)
			{
				bAddDetailedInfo = !FPlatformCrashContext::IsTypeContinuable(UnixContext->GetType());
			}

			// Program counters in the backtrace point to the location from where the execution will be resumed (in all frames except the one where we crashed),
			// which results in callstack pointing to the next lines in code. In order to determine the source line where the actual call happened, we need to go
			// back to the line that had the "call" instruction. Since x86(-64) instructions vary in length, we cannot do it reliably without disassembling,
			// just go back one byte - even if it's not the actual address of the call site.
			int OffsetToCallsite = CurrentCallDepth > 0 ? 1 : 0;

			FProgramCounterSymbolInfo TempSymbolInfo;

			// We can print detail info out during ensures, the only reason not to is if fail to populate the symbol info all the way
			bAddDetailedInfo = PopulateProgramCounterSymbolInfoFromSymbolFile(ProgramCounter - OffsetToCallsite, TempSymbolInfo);

			if (bAddDetailedInfo)
			{
				// append Module!FunctionName() [Source.cpp:X] to HumanReadableString
				FCStringAnsi::StrncatTruncateDest(HumanReadableString, HumanReadableStringSize, TempSymbolInfo.ModuleName);
				FCStringAnsi::StrncatTruncateDest(HumanReadableString, HumanReadableStringSize, "!");
				FCStringAnsi::StrncatTruncateDest(HumanReadableString, HumanReadableStringSize, TempSymbolInfo.FunctionName);
				FCStringAnsi::Sprintf(TempArray, " [%s:%d]", TempSymbolInfo.Filename, TempSymbolInfo.LineNumber);
				FCStringAnsi::StrncatTruncateDest(HumanReadableString, HumanReadableStringSize, TempArray);

				if (UnixContext)
				{
					// append Module!FunctioName [Source.cpp:X] to MinidumpCallstackInfo
					FCStringAnsi::StrncatTruncateDest(UnixContext->MinidumpCallstackInfo, UE_ARRAY_COUNT(UnixContext->MinidumpCallstackInfo), TempSymbolInfo.ModuleName);
					FCStringAnsi::StrncatTruncateDest(UnixContext->MinidumpCallstackInfo, UE_ARRAY_COUNT(UnixContext->MinidumpCallstackInfo), "!");
					FCStringAnsi::StrncatTruncateDest(UnixContext->MinidumpCallstackInfo, UE_ARRAY_COUNT(UnixContext->MinidumpCallstackInfo), TempSymbolInfo.FunctionName);
					FCStringAnsi::StrncatTruncateDest(UnixContext->MinidumpCallstackInfo, UE_ARRAY_COUNT(UnixContext->MinidumpCallstackInfo), TempArray);
				}
			}
			else
			{
				const char* ModuleName   = nullptr;
				const char* FunctionName = nullptr;

				// We have failed to fully populate the SymbolInfo, but we could still have basic information. Lets try to print as much info as possible
				if (TempSymbolInfo.ModuleName[0] != '\0')
				{
					ModuleName = TempSymbolInfo.ModuleName;
				}

				if (TempSymbolInfo.FunctionName[0] != '\0')
				{
					FunctionName = TempSymbolInfo.FunctionName;
				}

				FCStringAnsi::StrncatTruncateDest(HumanReadableString, HumanReadableStringSize, ModuleName != nullptr ? ModuleName : "");
				FCStringAnsi::StrncatTruncateDest(HumanReadableString, HumanReadableStringSize, "!");
				FCStringAnsi::StrncatTruncateDest(HumanReadableString, HumanReadableStringSize, FunctionName != nullptr ? FunctionName : "UnknownFunction");
				FCStringAnsi::StrncatTruncateDest(HumanReadableString, HumanReadableStringSize, FunctionName && TempSymbolInfo.SymbolDisplacement ? "(+": "(");

				if (UnixContext)
				{
					FCStringAnsi::StrncatTruncateDest(UnixContext->MinidumpCallstackInfo, UE_ARRAY_COUNT(UnixContext->MinidumpCallstackInfo), ModuleName != nullptr ? ModuleName : "Unknown");
					FCStringAnsi::StrncatTruncateDest(UnixContext->MinidumpCallstackInfo, UE_ARRAY_COUNT(UnixContext->MinidumpCallstackInfo), "!");
					FCStringAnsi::StrncatTruncateDest(UnixContext->MinidumpCallstackInfo, UE_ARRAY_COUNT(UnixContext->MinidumpCallstackInfo), FunctionName != nullptr ? FunctionName : "UnknownFunction");
					FCStringAnsi::StrncatTruncateDest(UnixContext->MinidumpCallstackInfo, UE_ARRAY_COUNT(UnixContext->MinidumpCallstackInfo), FunctionName && TempSymbolInfo.SymbolDisplacement ? "(+": "(");
				}

				if (TempSymbolInfo.SymbolDisplacement > 0x0)
				{
					FCStringAnsi::Sprintf(TempArray, "%p", TempSymbolInfo.SymbolDisplacement);
					FCStringAnsi::StrncatTruncateDest(HumanReadableString, HumanReadableStringSize, TempArray);

					if (UnixContext)
					{
						FCStringAnsi::StrncatTruncateDest(UnixContext->MinidumpCallstackInfo, UE_ARRAY_COUNT(UnixContext->MinidumpCallstackInfo), TempArray);
					}
				}

				FCStringAnsi::StrncatTruncateDest(HumanReadableString, HumanReadableStringSize, ")");

				if (UnixContext)
				{
					FCStringAnsi::StrncatTruncateDest(UnixContext->MinidumpCallstackInfo, UE_ARRAY_COUNT(UnixContext->MinidumpCallstackInfo), ")");
				}
			}

			if (UnixContext)
			{
				FCStringAnsi::StrncatTruncateDest(UnixContext->MinidumpCallstackInfo, UE_ARRAY_COUNT(UnixContext->MinidumpCallstackInfo), "\r\n");	// this one always uses Windows line terminators
			}
		}
		return true;
	}
	return true;
}

UE_AUTORTFM_ALWAYS_OPEN void FUnixPlatformStackWalk::StackWalkAndDump( ANSICHAR* HumanReadableString, SIZE_T HumanReadableStringSize, int32 IgnoreCount, void* Context )
{
	if (!CanProcessStackWalk(HumanReadableString, HumanReadableStringSize))
	{
		return;
	}
	TGuardValue<bool> ReentryGuard(GStackWalkInProgress, true);

	if (Context == nullptr)
	{
		FUnixCrashContext CrashContext(ECrashContextType::Crash, TEXT(""));
		CrashContext.InitFromSignal(0, nullptr, nullptr);
		CrashContext.FirstCrashHandlerFrame = static_cast<uint64*>(__builtin_return_address(0));
		FGenericPlatformStackWalk::StackWalkAndDump(HumanReadableString, HumanReadableStringSize, IgnoreCount, &CrashContext);
	}
	else
	{
		FGenericPlatformStackWalk::StackWalkAndDump(HumanReadableString, HumanReadableStringSize, IgnoreCount, Context);
	}
}

void FUnixPlatformStackWalk::StackWalkAndDump(ANSICHAR* HumanReadableString, SIZE_T HumanReadableStringSize, void* ProgramCounter, void* Context)
{
	if (!CanProcessStackWalk(HumanReadableString, HumanReadableStringSize))
	{
		return;
	}
	TGuardValue<bool> ReentryGuard(GStackWalkInProgress, true);

	FGenericPlatformStackWalk::StackWalkAndDump(HumanReadableString, HumanReadableStringSize, ProgramCounter, Context);
}

namespace
{
	/** Helper sets the ensure value in the context and guarantees it gets reset
	 * afterwards (even if an exception is thrown) */
	struct FLocalGuardHelper
	{
		FLocalGuardHelper(FUnixCrashContext* InContext, ECrashContextType NewType)
			: Context(InContext), OldType(Context->GetType())
		{
			Context->SetType(NewType);
		}
		~FLocalGuardHelper()
		{
			Context->SetType(OldType);
		}

	private:
		FUnixCrashContext* Context;
		ECrashContextType OldType;
	};
} // namespace

UE_AUTORTFM_ALWAYS_OPEN void FUnixPlatformStackWalk::StackWalkAndDumpEx(ANSICHAR* HumanReadableString, SIZE_T HumanReadableStringSize, int32 IgnoreCount, uint32 Flags, void* Context)
{
	const bool bHandlingEnsure = (Flags & EStackWalkFlags::FlagsUsedWhenHandlingEnsure) == EStackWalkFlags::FlagsUsedWhenHandlingEnsure;
	GHandlingEnsure = bHandlingEnsure;
	ECrashContextType HandlingType = bHandlingEnsure? ECrashContextType::Ensure : ECrashContextType::Crash;

	if (Context == nullptr)
	{
		FUnixCrashContext CrashContext(HandlingType, TEXT(""));
		CrashContext.InitFromSignal(0, nullptr, nullptr);
		CrashContext.FirstCrashHandlerFrame = static_cast<uint64*>(__builtin_return_address(0));
		FPlatformStackWalk::StackWalkAndDump(HumanReadableString, HumanReadableStringSize, IgnoreCount, &CrashContext);
	}
	else
	{
		FLocalGuardHelper Guard(reinterpret_cast<FUnixCrashContext*>(Context), HandlingType);
		FPlatformStackWalk::StackWalkAndDump(HumanReadableString, HumanReadableStringSize, IgnoreCount, Context);
	}

	GHandlingEnsure = false;
}

UE_AUTORTFM_ALWAYS_OPEN void FUnixPlatformStackWalk::StackWalkAndDumpEx(ANSICHAR* HumanReadableString, SIZE_T HumanReadableStringSize, void* ProgramCounter, uint32 Flags, void* Context)
{
	const bool bHandlingEnsure = (Flags & EStackWalkFlags::FlagsUsedWhenHandlingEnsure) == EStackWalkFlags::FlagsUsedWhenHandlingEnsure;
	GHandlingEnsure = bHandlingEnsure;
	ECrashContextType HandlingType = bHandlingEnsure? ECrashContextType::Ensure : ECrashContextType::Crash;

	if (Context == nullptr)
	{
		FUnixCrashContext CrashContext(HandlingType, TEXT(""));
		CrashContext.InitFromSignal(0, nullptr, nullptr);
		CrashContext.FirstCrashHandlerFrame = nullptr; // ProgramCounter will trim the callstack instead.
		FPlatformStackWalk::StackWalkAndDump(HumanReadableString, HumanReadableStringSize, ProgramCounter, &CrashContext);
	}
	else
	{
		FLocalGuardHelper Guard(reinterpret_cast<FUnixCrashContext*>(Context), HandlingType);
		FPlatformStackWalk::StackWalkAndDump(HumanReadableString, HumanReadableStringSize, ProgramCounter, Context);
	}

	GHandlingEnsure = false;
}

namespace
{
	uint32 OverwriteBacktraceWithRealCallstack(uint64* BackTrace, uint32 Size, uint64* FirstCrashHandlerFrame)
	{
		if (!GFullCrashCallstack && Size && FirstCrashHandlerFrame != nullptr)
		{
			for (int i = 0; i < Size - 1; i++)
			{
				if (FirstCrashHandlerFrame == reinterpret_cast<uint64*>(BackTrace[i]))
				{
					i++;
					uint64* OverwriteBackTrace = BackTrace;

					for (int j = i; j < Size; j++)
					{
						*OverwriteBackTrace = BackTrace[j];
						OverwriteBackTrace++;
					}

					return Size - i;
				}
			}
		}

		return Size;
	}
}

UE_AUTORTFM_ALWAYS_OPEN uint32 FUnixPlatformStackWalk::CaptureStackBackTrace( uint64* BackTrace, uint32 MaxDepth, void* Context )
{
	// Make sure we have place to store the information before we go through the process of raising
	// an exception and handling it.
	if( BackTrace == nullptr || MaxDepth == 0 )
	{
		return 0;
	}

	size_t Size = backtrace(reinterpret_cast<void**>(BackTrace), MaxDepth);

	FUnixCrashContext* UnixContext = reinterpret_cast<FUnixCrashContext*>(Context);

	if (UnixContext)
	{
		return OverwriteBacktraceWithRealCallstack(BackTrace, Size, UnixContext->FirstCrashHandlerFrame);
	}

	return (uint32)Size;
}

UE_AUTORTFM_ALWAYS_OPEN uint32 FUnixPlatformStackWalk::CaptureStackBackTraceViaFramePointerWalking(uint64* BackTrace, uint32 MaxDepth)
{
#if defined(__aarch64__) || defined(__x86_64__)
	uint64 StackTopPtr = 0;

	// Cache the main thread's stack top to avoid repeated pthread_getattr_np calls.
	const bool bIsMainThread = syscall(SYS_gettid) == getpid();
	static std::atomic<uint64> StackTopMainThread{0};
	uint64 CachedStackTop = StackTopMainThread.load(std::memory_order_relaxed);
	if (bIsMainThread && CachedStackTop != 0)
	{
		StackTopPtr = CachedStackTop;
	}
	else
	{
		pthread_attr_t ThreadAttr;
		if (pthread_getattr_np(pthread_self(), &ThreadAttr) != 0)
		{
			return 0;
		}

		void* StackBase;
		size_t StackSize;
		if (pthread_attr_getstack(&ThreadAttr, &StackBase, &StackSize) != 0)
		{
			pthread_attr_destroy(&ThreadAttr);
			return 0;
		}

		pthread_attr_destroy(&ThreadAttr);

		StackTopPtr = (uint64)StackBase + StackSize;

		if (bIsMainThread)
		{
			StackTopMainThread.store(StackTopPtr, std::memory_order_relaxed);
		}
	}

	struct StackFrame
	{
		StackFrame* NextFrame;
		UPTRINT ReturnPtr;

		inline UPTRINT GetReturnPtr()
		{
#if defined(__aarch64__)
			uintptr_t Ptr = ReturnPtr;
			// xpaclri operates on the link register (x30), so move value there and back.
			// This instruction is HINT #7, mapped to NOP on pre-PAC architectures.
			asm volatile(
				"mov x30, %0\n"
				"xpaclri\n"
				"mov %0, x30\n"
				: "+r"(Ptr)
				:
				: "x30"
			);
			return Ptr;
#else
			return ReturnPtr;
#endif
		}
	};

	StackFrame* FrameStart = (StackFrame*)__builtin_frame_address(0);
	if (FrameStart == nullptr)
	{
		return 0;
	}

	StackFrame* FrameEnd = (StackFrame*)StackTopPtr;
	uint32 NumStackFrames = 0;

	for (StackFrame* CurrentFrame = FrameStart;
		NumStackFrames < MaxDepth &&
		CurrentFrame->NextFrame > FrameStart &&
		CurrentFrame->NextFrame < FrameEnd &&
		((UPTRINT)CurrentFrame->NextFrame & (sizeof(StackFrame) - 1)) == 0 && // stop at unaligned frame
		CurrentFrame->GetReturnPtr() != 0; // stop if function ptr is 0
		CurrentFrame = CurrentFrame->NextFrame, NumStackFrames++)
	{
		BackTrace[NumStackFrames] = CurrentFrame->GetReturnPtr();
	}

	return NumStackFrames;
#else
	return 0;
#endif
}

namespace
{
	// Shorter timeout for crash handling — the default CVar is 60s which would cause
	// multi-minute waits if several threads are unresponsive.
	constexpr double CrashHandlingThreadStackMaxWait = 2.0;

	void WaitForSignalHandlerToFinishOrCrash(ThreadStackUserData& ThreadStack)
	{
		const double MaxWait = GUnixStackWalkNoCrashOnTimeout
			? CrashHandlingThreadStackMaxWait
			: CVarUnixPlatformThreadCallStackMaxWait.AsVariable()->GetFloat();
		double EndWaitTimestamp = FPlatformTime::Seconds() + MaxWait;
		double CurrentTimestamp = FPlatformTime::Seconds();

		while (!ThreadStack.bDone)
		{
			if (CurrentTimestamp > EndWaitTimestamp)
			{
				if (GUnixStackWalkNoCrashOnTimeout)
				{
					// During crash handling, skip unresponsive threads rather than causing a recursive crash
					return;
				}
				// We have waited for as long as we should for the signal handler to finish. Assume it has hang and we need to kill our selfs
				*(int*)0x10 = 0x0;
			}

			CurrentTimestamp = FPlatformTime::Seconds();
		}
	}

	void GatherCallstackFromThread(ThreadStackUserData& ThreadStack, uint64 ThreadId)
	{
		sigval UserData;
		UserData.sival_ptr = &ThreadStack;

		siginfo_t info;
		memset (&info, 0, sizeof (siginfo_t));
		info.si_signo = THREAD_CALLSTACK_GENERATOR;
		info.si_code  = SI_QUEUE;
		info.si_pid   = syscall(SYS_getpid);
		info.si_uid   = syscall(SYS_getuid);
		info.si_value = UserData;

		// Avoid using sigqueue here as if the ThreadId is already blocked and in a signal handler
		// sigqueue will try a different thread signal handler and report the wrong callstack
		if (syscall(SYS_rt_tgsigqueueinfo, info.si_pid, ThreadId, THREAD_CALLSTACK_GENERATOR, &info) == 0)
		{
			WaitForSignalHandlerToFinishOrCrash(ThreadStack);

			if (!ThreadStack.bDone && GUnixStackWalkNoCrashOnTimeout)
			{
				// Timed out waiting for signal handler. Explicitly mark as failed so
				// callers see a deterministic 0 count rather than stale/partial data.
				ThreadStack.BackTraceCount = 0;
			}
		}
	}
}

UE_AUTORTFM_ALWAYS_OPEN void FUnixPlatformStackWalk::ThreadStackWalkAndDump(ANSICHAR* HumanReadableString, SIZE_T HumanReadableStringSize, int32 IgnoreCount, uint32 ThreadId)
{
	if (!CanProcessStackWalk(HumanReadableString, HumanReadableStringSize))
	{
		return;
	}
	TGuardValue<bool> ReentryGuard(GStackWalkInProgress, true);

	ThreadStackUserData ThreadCallStack;
	ThreadCallStack.bCaptureCallStack = true;
	ThreadCallStack.CallStackSize     = HumanReadableStringSize;
	ThreadCallStack.CallStack         = HumanReadableString;
	ThreadCallStack.BackTraceCount    = 0;
	ThreadCallStack.bDone             = false;

	GatherCallstackFromThread(ThreadCallStack, ThreadId);
}

UE_AUTORTFM_ALWAYS_OPEN uint32 FUnixPlatformStackWalk::CaptureThreadStackBackTrace(uint64 ThreadId, uint64* BackTrace, uint32 MaxDepth, void* Context)
{
	// During crash handling (GUnixStackWalkNoCrashOnTimeout == true), use a ring buffer of
	// static entries so that if the signal times out and we return early, the in-flight signal
	// handler on the target thread still has valid memory to write to (avoiding use-after-scope
	// on a stack local). The ring buffer ensures each timed-out capture gets its own slot,
	// preventing a late-arriving handler from corrupting the current capture's data.
	static constexpr uint32 RingBufferSize = 8;
	static ThreadStackUserData StaticRingBuffer[RingBufferSize];
	static uint32 RingBufferIndex = 0;
	ThreadStackUserData StackThreadBackTrace;

	ThreadStackUserData& ThreadBackTrace = GUnixStackWalkNoCrashOnTimeout
		? StaticRingBuffer[RingBufferIndex++ % RingBufferSize]
		: StackThreadBackTrace;
	ThreadBackTrace.bCaptureCallStack = false;
	ThreadBackTrace.CallStackSize     = MaxDepth;
	ThreadBackTrace.BackTrace         = BackTrace;
	ThreadBackTrace.BackTraceCount    = 0;
	ThreadBackTrace.bDone             = false;

	GatherCallstackFromThread(ThreadBackTrace, ThreadId);

	// The signal handler will set this value, we just have to make sure we wait for the signal handler we raised to finish
	return ThreadBackTrace.BackTraceCount;
}

namespace
{
	int NumberOfDynamicLibrariesCallback(struct dl_phdr_info* Info, size_t /*size*/, void* Data)
	{
		int* Size = static_cast<int*>(Data);
		if (Info->dlpi_name)
		{
			(*Size)++;
		}

		// Continute until no more callbacks
		return 0;
	}
}

UE_AUTORTFM_ALWAYS_OPEN int32 FUnixPlatformStackWalk::GetProcessModuleCount()
{
	int Size = 0;
	dl_iterate_phdr(NumberOfDynamicLibrariesCallback, &Size);

	return Size;
}

namespace
{
	struct ProcessModuleSignatures
	{
		FStackWalkModuleInfo* ModuleSignatures;
		int32 ModuleSignaturesSize;
		int32 Index;
	};

	int CollectModuleSignatures(struct dl_phdr_info* Info, size_t /*size*/, void* Data)
	{
		ProcessModuleSignatures* Modules = static_cast<ProcessModuleSignatures*>(Data);

		if (Info->dlpi_name)
		{
			int TotalMemSize = 0;
			uint64 RealBase = 0;
			bool bRealBaseSet = false;
			for (int i = 0; i < Info->dlpi_phnum; i++)
			{
				TotalMemSize += Info->dlpi_phdr[i].p_memsz;

				// Lets get our real base from the BASE + first LOAD segment
				if (!bRealBaseSet && Info->dlpi_phdr[i].p_type == PT_LOAD)
				{
					RealBase = (uint64)(Info->dlpi_addr + Info->dlpi_phdr[i].p_vaddr);
					bRealBaseSet = true;
				}
			}

			FString ImageName = FPaths::GetCleanFilename(Info->dlpi_name);

			// If Info->dlpi_name is empty then it is the binary name
			if (ImageName.IsEmpty())
			{
				ImageName = FPlatformProcess::ExecutableName(false);
			}

			FStackWalkModuleInfo StackInfo;
			memset(&StackInfo, 0, sizeof(StackInfo));
			StackInfo.BaseOfImage = RealBase;
			// TODO Check if ImageName is greater then 32 bytes, if so we need to look at increasing the struct size
			FCString::Strcpy(StackInfo.ImageName, *ImageName);
			StackInfo.ImageSize = TotalMemSize;
			FCString::Strcpy(StackInfo.LoadedImageName, *ImageName);
			FCString::Strcpy(StackInfo.ModuleName, *ImageName);
			FMemory::Memzero(&StackInfo.PdbSig70, sizeof(StackInfo.PdbSig70));

			Modules->ModuleSignatures[Modules->Index] = StackInfo;
			Modules->Index++;
		}

		// Continute until our index is less then our size or no more callbacks
		return Modules->Index >= Modules->ModuleSignaturesSize;
	}
}

UE_AUTORTFM_ALWAYS_OPEN int32 FUnixPlatformStackWalk::GetProcessModuleSignatures(FStackWalkModuleInfo *ModuleSignatures, const int32 ModuleSignaturesSize)
{
	if (ModuleSignatures == nullptr || ModuleSignaturesSize == 0)
	{
		return 0;
	}

	ProcessModuleSignatures Signatures{ModuleSignatures, ModuleSignaturesSize, 0};
	dl_iterate_phdr(CollectModuleSignatures, &Signatures);

	return Signatures.Index;
}

thread_local const TCHAR* GCrashErrorMessage = nullptr;
thread_local void* GCrashErrorProgramCounter = nullptr;
thread_local ECrashContextType GCrashErrorType = ECrashContextType::Crash;

void ReportAssert(const TCHAR* ErrorMessage, void* ProgramCounter)
{
	GCrashErrorMessage = ErrorMessage;
	GCrashErrorProgramCounter = ProgramCounter;
	GCrashErrorType = ECrashContextType::Assert;

	FPlatformMisc::RaiseException(1);
}

void ReportGPUCrash(const TCHAR* ErrorMessage, void* ProgramCounter)
{
	if (ProgramCounter == nullptr)
	{
		ProgramCounter = PLATFORM_RETURN_ADDRESS();
	}

	GCrashErrorMessage = ErrorMessage;
	GCrashErrorProgramCounter = ProgramCounter;
	GCrashErrorType = ECrashContextType::GPUCrash;

	FPlatformMisc::RaiseException(1);
}

static FCriticalSection ReportLock;
static bool bReentranceGuard = false;

void ReportEnsure(const TCHAR* ErrorMessage, void* ProgramCounter)
{
	// Simple re-entrance guard.
	ReportLock.Lock();

	if (bReentranceGuard)
	{
		ReportLock.Unlock();
		return;
	}

	bReentranceGuard = true;

	FUnixCrashContext EnsureContext(ECrashContextType::Ensure, ErrorMessage);
	EnsureContext.InitFromDiagnostics(ProgramCounter);

	EnsureContext.CaptureStackTrace(ProgramCounter);
	EnsureContext.GenerateCrashInfoAndLaunchReporter();

	bReentranceGuard = false;
	ReportLock.Unlock();
}

void ReportStall(const TCHAR* Message, uint32 ThreadId)
{
	// Simple re-entrance guard.
	ReportLock.Lock();

	if (bReentranceGuard)
	{
		ReportLock.Unlock();
		return;
	}

	bReentranceGuard = true;

	FUnixCrashContext StallContext(ECrashContextType::Stall, Message);
	StallContext.InitFromDiagnostics();

	StallContext.CaptureThreadStackTrace(ThreadId);
	StallContext.GenerateCrashInfoAndLaunchReporter();

	bReentranceGuard = false;
	ReportLock.Unlock();
}

void ReportHang(const TCHAR* ErrorMessage, const uint64* StackFrames, int32 NumStackFrames, uint32 HungThreadId)
{
	ReportLock.Lock();
	if (!bReentranceGuard)
	{
		bReentranceGuard = true;

		FUnixCrashContext HangContext(ECrashContextType::Hang, ErrorMessage);
		HangContext.SetPortableCallStack(StackFrames, NumStackFrames);
		HangContext.GenerateCrashInfoAndLaunchReporter();

		bReentranceGuard = false;
	}
	ReportLock.Unlock();
}
