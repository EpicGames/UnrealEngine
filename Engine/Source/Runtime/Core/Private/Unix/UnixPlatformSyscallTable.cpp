// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnixPlatformSyscallTable.h"
#include "Containers/UnrealString.h"

namespace
{
	struct FSyscallEntry
	{
		const TCHAR* const SyscallName;
		const int SyscallNumber;
	};
}

int UnixPlatformLookupSyscallNumberFromName(const FString& SyscallName)
{
	// Our sysroot is glibc 2.17 which means we dont have a full picture of all the newer syscall defines/setups. So better to just
	// manually define our own look up table vs depending on manually defining missing ones for newer kernels
	//
	// Use just a linear look-up as our N is small
	static FSyscallEntry SyscallNameToNumber[] = {
		// These syscall numbers are from the kernel
#if PLATFORM_CPU_X86_FAMILY
		{TEXT("read"),		0},
		{TEXT("write"),		1},
		{TEXT("open"),		2},
		{TEXT("close"),		3},
		{TEXT("stat"),		4},
		{TEXT("fstat"),		5},
		{TEXT("lstat"),		6},
		{TEXT("poll"),		7},
		{TEXT("lseek"),		8},
		{TEXT("mmap"),		9},
		{TEXT("mprotect"),	10},
		{TEXT("munmap"),	11},
		{TEXT("brk"),		12},
		{TEXT("rt_sigaction"),	13},
		{TEXT("rt_sigprocmask"),14},
		{TEXT("rt_sigreturn"),	15},
		{TEXT("ioctl"),		16},
		{TEXT("pread64"),	17},
		{TEXT("pwrite64"),	18},
		{TEXT("readv"),		19},
		{TEXT("writev"),	20},
		{TEXT("access"),	21},
		{TEXT("pipe"),		22},
		{TEXT("select"),	23},
		{TEXT("sched_yield"), 24},
		{TEXT("mremap"),	25},
		{TEXT("msync"),		26},
		{TEXT("mincore"),	27},
		{TEXT("madvise"),	28},
		{TEXT("shmget"),	29},
		{TEXT("shmat"),		30},
		{TEXT("shmctl"),	31},
		{TEXT("dup"),		32},
		{TEXT("dup2"),		33},
		{TEXT("pause"),		34},
		{TEXT("nanosleep"),	35},
		{TEXT("getitimer"),	36},
		{TEXT("alarm"),		37},
		{TEXT("setitimer"),	38},
		{TEXT("getpid"),	39},
		{TEXT("sendfile"),	40},
		{TEXT("socket"),	41},
		{TEXT("connect"),	42},
		{TEXT("accept"),	43},
		{TEXT("sendto"),	44},
		{TEXT("recvfrom"),	45},
		{TEXT("sendmsg"),	46},
		{TEXT("recvmsg"),	47},
		{TEXT("shutdown"),	48},
		{TEXT("bind"),		49},
		{TEXT("listen"),	50},
		{TEXT("getsockname"),	51},
		{TEXT("getpeername"),	52},
		{TEXT("socketpair"),	53},
		{TEXT("setsockopt"),	54},
		{TEXT("getsockopt"),	55},
		{TEXT("clone"),		56},
		{TEXT("fork"),		57},
		{TEXT("vfork"),		58},
		{TEXT("execve"),	59},
		{TEXT("exit"),		60},
		{TEXT("wait4"),		61},
		{TEXT("kill"),		62},
		{TEXT("uname"),		63},
		{TEXT("semget"),	64},
		{TEXT("semop"),		65},
		{TEXT("semctl"),	66},
		{TEXT("shmdt"),		67},
		{TEXT("msgget"),	68},
		{TEXT("msgsnd"),	69},
		{TEXT("msgrcv"),	70},
		{TEXT("msgctl"),	71},
		{TEXT("fcntl"),		72},
		{TEXT("flock"),		73},
		{TEXT("fsync"),		74},
		{TEXT("fdatasync"),	75},
		{TEXT("truncate"),	76},
		{TEXT("ftruncate"),	77},
		{TEXT("getdents"),	78},
		{TEXT("getcwd"),	79},
		{TEXT("chdir"),		80},
		{TEXT("fchdir"),	81},
		{TEXT("rename"),	82},
		{TEXT("mkdir"),		83},
		{TEXT("rmdir"),		84},
		{TEXT("creat"),		85},
		{TEXT("link"),		86},
		{TEXT("unlink"),	87},
		{TEXT("symlink"),	88},
		{TEXT("readlink"),	89},
		{TEXT("chmod"),		90},
		{TEXT("fchmod"),	91},
		{TEXT("chown"),		92},
		{TEXT("fchown"),	93},
		{TEXT("lchown"),	94},
		{TEXT("umask"),		95},
		{TEXT("gettimeofday"),	96},
		{TEXT("getrlimit"),		97},
		{TEXT("getrusage"),		98},
		{TEXT("sysinfo"),		99},
		{TEXT("times"),			100},
		{TEXT("ptrace"),		101},
		{TEXT("getuid"),		102},
		{TEXT("syslog"),		103},
		{TEXT("getgid"),		104},
		{TEXT("setuid"),		105},
		{TEXT("setgid"),		106},
		{TEXT("geteuid"),		107},
		{TEXT("getegid"),		108},
		{TEXT("setpgid"),		109},
		{TEXT("getppid"),		110},
		{TEXT("getpgrp"),		111},
		{TEXT("setsid"),		112},
		{TEXT("setreuid"),		113},
		{TEXT("setregid"),		114},
		{TEXT("getgroups"),		115},
		{TEXT("setgroups"),		116},
		{TEXT("setresuid"),		117},
		{TEXT("getresuid"),		118},
		{TEXT("setresgid"),		119},
		{TEXT("getresgid"),		120},
		{TEXT("getpgid"),		121},
		{TEXT("setfsuid"),		122},
		{TEXT("setfsgid"),		123},
		{TEXT("getsid"),		124},
		{TEXT("capget"),		125},
		{TEXT("capset"),		126},
		{TEXT("rt_sigpending"),	127},
		{TEXT("rt_sigtimedwait"),	128},
		{TEXT("rt_sigqueueinfo"),	129},
		{TEXT("rt_sigsuspend"),		130},
		{TEXT("sigaltstack"),		131},
		{TEXT("utime"),		132},
		{TEXT("mknod"),		133},
		{TEXT("uselib"),	134},
		{TEXT("personality"), 135},
		{TEXT("ustat"),		136},
		{TEXT("statfs"),	137},
		{TEXT("fstatfs"),	138},
		{TEXT("sysfs"),		139},
		{TEXT("getpriority"),	140},
		{TEXT("setpriority"),	141},
		{TEXT("sched_setparam"),	142},
		{TEXT("sched_getparam"),	143},
		{TEXT("sched_setscheduler"),	144},
		{TEXT("sched_getscheduler"),	145},
		{TEXT("sched_get_priority_max"),	146},
		{TEXT("sched_get_priority_min"),	147},
		{TEXT("sched_rr_get_interval"),		148},
		{TEXT("mlock"),			149},
		{TEXT("munlock"),		150},
		{TEXT("mlockall"),		151},
		{TEXT("munlockall"),	152},
		{TEXT("vhangup"),		153},
		{TEXT("modify_ldt"),	154},
		{TEXT("pivot_root"),	155},
		{TEXT("_sysctl"),		156},
		{TEXT("prctl"),			157},
		{TEXT("arch_prctl"),	158},
		{TEXT("adjtimex"),		159},
		{TEXT("setrlimit"),		160},
		{TEXT("chroot"),		161},
		{TEXT("sync"),			162},
		{TEXT("acct"),			163},
		{TEXT("settimeofday"),	164},
		{TEXT("mount"),			165},
		{TEXT("umount2"),		166},
		{TEXT("swapon"),		167},
		{TEXT("swapoff"),		168},
		{TEXT("reboot"),		169},
		{TEXT("sethostname"),	170},
		{TEXT("setdomainname"),	171},
		{TEXT("iopl"),			172},
		{TEXT("ioperm"),		173},
		{TEXT("create_module"),	174},
		{TEXT("init_module"),	175},
		{TEXT("delete_module"),	176},
		{TEXT("get_kernel_syms"),	177},
		{TEXT("query_module"),	178},
		{TEXT("quotactl"),		179},
		{TEXT("nfsservctl"),	180},
		{TEXT("getpmsg"),		181},
		{TEXT("putpmsg"),		182},
		{TEXT("afs_syscall"),	183},
		{TEXT("tuxcall"),		184},
		{TEXT("security"),		185},
		{TEXT("gettid"),		186},
		{TEXT("readahead"),		187},
		{TEXT("setxattr"),		188},
		{TEXT("lsetxattr"),		189},
		{TEXT("fsetxattr"),		190},
		{TEXT("getxattr"),		191},
		{TEXT("lgetxattr"),		192},
		{TEXT("fgetxattr"),		193},
		{TEXT("listxattr"),		194},
		{TEXT("llistxattr"),	195},
		{TEXT("flistxattr"),	196},
		{TEXT("removexattr"),	197},
		{TEXT("lremovexattr"),	198},
		{TEXT("fremovexattr"),	199},
		{TEXT("tkill"),		200},
		{TEXT("time"),		201},
		{TEXT("futex"),		202},
		{TEXT("sched_setaffinity"),	203},
		{TEXT("sched_getaffinity"),	204},
		{TEXT("set_thread_area"),	205},
		{TEXT("io_setup"),		206},
		{TEXT("io_destroy"),	207},
		{TEXT("io_getevents"),	208},
		{TEXT("io_submit"),		209},
		{TEXT("io_cancel"),		210},
		{TEXT("get_thread_area"),	211},
		{TEXT("lookup_dcookie"),	212},
		{TEXT("epoll_create"),		213},
		{TEXT("epoll_ctl_old"),		214},
		{TEXT("epoll_wait_old"),	215},
		{TEXT("remap_file_pages"),	216},
		{TEXT("getdents64"),		217},
		{TEXT("set_tid_address"),	218},
		{TEXT("restart_syscall"),	219},
		{TEXT("semtimedop"),		220},
		{TEXT("fadvise64"),			221},
		{TEXT("timer_create"),		222},
		{TEXT("timer_settime"),		223},
		{TEXT("timer_gettime"),		224},
		{TEXT("timer_getoverrun"),	225},
		{TEXT("timer_delete"),		226},
		{TEXT("clock_settime"),		227},
		{TEXT("clock_gettime"),		228},
		{TEXT("clock_getres"),		229},
		{TEXT("clock_nanosleep"),	230},
		{TEXT("exit_group"),		231},
		{TEXT("epoll_wait"),		232},
		{TEXT("epoll_ctl"),			233},
		{TEXT("tgkill"),			234},
		{TEXT("utimes"),			235},
		{TEXT("vserver"),			236},
		{TEXT("mbind"),				237},
		{TEXT("set_mempolicy"),		238},
		{TEXT("get_mempolicy"),		239},
		{TEXT("mq_open"),			240},
		{TEXT("mq_unlink"),			241},
		{TEXT("mq_timedsend"),		242},
		{TEXT("mq_timedreceive"),	243},
		{TEXT("mq_notify"),			244},
		{TEXT("mq_getsetattr"),		245},
		{TEXT("kexec_load"),		246},
		{TEXT("waitid"),			247},
		{TEXT("add_key"),			248},
		{TEXT("request_key"),		249},
		{TEXT("keyctl"),			250},
		{TEXT("ioprio_set"),		251},
		{TEXT("ioprio_get"),		252},
		{TEXT("inotify_init"),		253},
		{TEXT("inotify_add_watch"),	254},
		{TEXT("inotify_rm_watch"),	255},
		{TEXT("migrate_pages"),		256},
		{TEXT("openat"),		257},
		{TEXT("mkdirat"),		258},
		{TEXT("mknodat"),		259},
		{TEXT("fchownat"),		260},
		{TEXT("futimesat"),		261},
		{TEXT("newfstatat"),	262},
		{TEXT("unlinkat"),		263},
		{TEXT("renameat"),		264},
		{TEXT("linkat"),		265},
		{TEXT("symlinkat"),		266},
		{TEXT("readlinkat"),	267},
		{TEXT("fchmodat"),		268},
		{TEXT("faccessat"),		269},
		{TEXT("pselect6"),		270},
		{TEXT("ppoll"),			271},
		{TEXT("unshare"),		272},
		{TEXT("set_robust_list"),	273},
		{TEXT("get_robust_list"),	274},
		{TEXT("splice"),			275},
		{TEXT("tee"),				276},
		{TEXT("sync_file_range"),	277},
		{TEXT("vmsplice"),			278},
		{TEXT("move_pages"),		279},
		{TEXT("utimensat"),			280},
		{TEXT("epoll_pwait"),		281},
		{TEXT("signalfd"),			282},
		{TEXT("timerfd_create"),	283},
		{TEXT("eventfd"),			284},
		{TEXT("fallocate"),			285},
		{TEXT("timerfd_settime"),	286},
		{TEXT("timerfd_gettime"),	287},
		{TEXT("accept4"),		288},
		{TEXT("signalfd4"),		289},
		{TEXT("eventfd2"),		290},
		{TEXT("epoll_create1"),	291},
		{TEXT("dup3"),			292},
		{TEXT("pipe2"),			293},
		{TEXT("inotify_init1"),	294},
		{TEXT("preadv"),		295},
		{TEXT("pwritev"),		296},
		{TEXT("rt_tgsigqueueinfo"),	297},
		{TEXT("perf_event_open"),	298},
		{TEXT("recvmmsg"),			299},
		{TEXT("fanotify_init"),		300},
		{TEXT("fanotify_mark"),		301},
		{TEXT("prlimit64"),			302},
		{TEXT("name_to_handle_at"),	303},
		{TEXT("open_by_handle_at"),	304},
		{TEXT("clock_adjtime"),		305},
		{TEXT("syncfs"),			306},
		{TEXT("sendmmsg"),			307},
		{TEXT("setns"),				308},
		{TEXT("getcpu"),			309},
		{TEXT("process_vm_readv"),	310},
		{TEXT("process_vm_writev"),	311},
		{TEXT("kcmp"),				312},
		{TEXT("finit_module"),		313},
		{TEXT("sched_setattr"),		314},
		{TEXT("sched_getattr"),		315},
		{TEXT("renameat2"),			316},
		{TEXT("seccomp"),			317},
		{TEXT("getrandom"),			318},
		{TEXT("memfd_create"),		319},
		{TEXT("kexec_file_load"),	320},
		{TEXT("bpf"),				321},
		{TEXT("execveat"),			322},
		{TEXT("userfaultfd"),		323},
		{TEXT("membarrier"),		324},
		{TEXT("mlock2"),			325},
		{TEXT("copy_file_range"),	326},
		{TEXT("preadv2"),			327},
		{TEXT("pwritev2"),			328},
		{TEXT("pkey_mprotect"),		329},
		{TEXT("pkey_alloc"),		330},
		{TEXT("pkey_free"),			331},
		{TEXT("statx"),				332},
		{TEXT("io_pgetevents"),		333},
		{TEXT("rseq"),				334},
		{TEXT("pidfd_send_signal"), 424},
		{TEXT("io_uring_setup"),	425},
		{TEXT("io_uring_enter"),	426},
		{TEXT("io_uring_register"),	427},
		{TEXT("open_tree"),			428},
		{TEXT("move_mount"),		429},
		{TEXT("fsopen"),			430},
		{TEXT("fsconfig"),			431},
		{TEXT("fsmount"),			432},
		{TEXT("fspick"),			433},
		{TEXT("pidfd_open"),		434},
		{TEXT("clone3"),			435},
		{TEXT("close_range"),		436},
		{TEXT("openat2"),			437},
		{TEXT("pidfd_getfd"),		438},
		{TEXT("faccessat2"),		439},
		{TEXT("process_madvise"), 	440},
		{TEXT("epoll_pwait2"),		441},
		{TEXT("mount_setattr"),		442},
		{TEXT("quotactl_fd"),		443},
		{TEXT("landlock_create_ruleset"), 444},
		{TEXT("landlock_add_rule"), 445},
		{TEXT("landlock_restrict_self"),  446},
		{TEXT("memfd_secret"),		447},
		{TEXT("process_mrelease"),	448},

#elif PLATFORM_CPU_ARM_FAMILY
		{TEXT("io_setup"),		0},
		{TEXT("io_destroy"),	1},
		{TEXT("io_submit"),		2},
		{TEXT("io_cancel"),		3},
		{TEXT("io_getevents"),	4},
		{TEXT("setxattr"),		5},
		{TEXT("lsetxattr"),		6},
		{TEXT("fsetxattr"),		7},
		{TEXT("getxattr"),		8},
		{TEXT("lgetxattr"),		9},
		{TEXT("fgetxattr"),		10},
		{TEXT("listxattr"),		11},
		{TEXT("llistxattr"),	12},
		{TEXT("flistxattr"),	13},
		{TEXT("removexattr"),	14},
		{TEXT("lremovexattr"),	15},
		{TEXT("fremovexattr"),	16},
		{TEXT("getcwd"),		17},
		{TEXT("lookup_dcookie"),18},
		{TEXT("eventfd2"),		19},
		{TEXT("epoll_create1"),	20},
		{TEXT("epoll_ctl"),		21},
		{TEXT("epoll_pwait"),	22},
		{TEXT("dup"),		23},
		{TEXT("dup3"),		24},
		{TEXT("fcntl"),		25},
		{TEXT("inotify_init1"),		26},
		{TEXT("inotify_add_watch"),	27},
		{TEXT("inotify_rm_watch"),	28},
		{TEXT("ioctl"),				29},
		{TEXT("ioprio_set"),		30},
		{TEXT("ioprio_get"),		31},
		{TEXT("flock"),			32},
		{TEXT("mknodat"),		33},
		{TEXT("mkdirat"),		34},
		{TEXT("unlinkat"),		35},
		{TEXT("symlinkat"),		36},
		{TEXT("linkat"),		37},
		{TEXT("renameat"),		38},
		{TEXT("umount2"),		39},
		{TEXT("mount"),			40},
		{TEXT("pivot_root"),	41},
		{TEXT("nfsservctl"),	42},
		{TEXT("statfs"),		43},
		{TEXT("fstatfs"),		44},
		{TEXT("truncate"),		45},
		{TEXT("ftruncate"),		46},
		{TEXT("fallocate"),		47},
		{TEXT("faccessat"),		48},
		{TEXT("chdir"),			49},
		{TEXT("fchdir"),		50},
		{TEXT("chroot"),		51},
		{TEXT("fchmod"),		52},
		{TEXT("fchmodat"),		53},
		{TEXT("fchownat"),		54},
		{TEXT("fchown"),		55},
		{TEXT("openat"),		56},
		{TEXT("close"),			57},
		{TEXT("vhangup"),		58},
		{TEXT("pipe2"),			59},
		{TEXT("quotactl"),		60},
		{TEXT("getdents64"),	61},
		{TEXT("lseek"),		62},
		{TEXT("read"),		63},
		{TEXT("write"),		64},
		{TEXT("readv"),		65},
		{TEXT("writev"),	66},
		{TEXT("pread64"),	67},
		{TEXT("pwrite64"),	68},
		{TEXT("preadv"),	69},
		{TEXT("pwritev"),	70},
		{TEXT("sendfile"),	71},
		{TEXT("pselect6"),	72},
		{TEXT("ppoll"),		73},
		{TEXT("signalfd4"),	74},
		{TEXT("vmsplice"),	75},
		{TEXT("splice"),	76},
		{TEXT("tee"),		77},
		{TEXT("readlinkat"),78},
		{TEXT("newfstatat"),79},
		{TEXT("fstat"),		80},
		{TEXT("sync"),		81},
		{TEXT("fsync"),		82},
		{TEXT("fdatasync"),	83},
		{TEXT("sync_file_range"),	84},
		{TEXT("timerfd_create"),	85},
		{TEXT("timerfd_settime"),	86},
		{TEXT("timerfd_gettime"),	87},
		{TEXT("utimensat"),		88},
		{TEXT("acct"),			89},
		{TEXT("capget"),		90},
		{TEXT("capset"),		91},
		{TEXT("personality"),	92},
		{TEXT("exit"),			93},
		{TEXT("exit_group"),	94},
		{TEXT("waitid"),		95},
		{TEXT("set_tid_address"),	96},
		{TEXT("unshare"),			97},
		{TEXT("futex"),				98},
		{TEXT("set_robust_list"),	99},
		{TEXT("get_robust_list"),	100},
		{TEXT("nanosleep"),			101},
		{TEXT("getitimer"),			102},
		{TEXT("setitimer"),			103},
		{TEXT("kexec_load"),		104},
		{TEXT("init_module"),		105},
		{TEXT("delete_module"),		106},
		{TEXT("timer_create"),		107},
		{TEXT("timer_gettime"),		108},
		{TEXT("timer_getoverrun"),	109},
		{TEXT("timer_settime"),		110},
		{TEXT("timer_delete"),		111},
		{TEXT("clock_settime"),		112},
		{TEXT("clock_gettime"),		113},
		{TEXT("clock_getres"),		114},
		{TEXT("clock_nanosleep"),	115},
		{TEXT("syslog"),			116},
		{TEXT("ptrace"),			117},
		{TEXT("sched_setparam"),	118},
		{TEXT("sched_setscheduler"),119},
		{TEXT("sched_getscheduler"),120},
		{TEXT("sched_getparam"),	121},
		{TEXT("sched_setaffinity"),	122},
		{TEXT("sched_getaffinity"),	123},
		{TEXT("sched_yield"),		124},
		{TEXT("sched_get_priority_max"),	125},
		{TEXT("sched_get_priority_min"),	126},
		{TEXT("sched_rr_get_interval"),		127},
		{TEXT("restart_syscall"),			128},
		{TEXT("kill"),				129},
		{TEXT("tkill"),				130},
		{TEXT("tgkill"),			131},
		{TEXT("sigaltstack"),		132},
		{TEXT("rt_sigsuspend"),		133},
		{TEXT("rt_sigaction"),		134},
		{TEXT("rt_sigprocmask"),	135},
		{TEXT("rt_sigpending"),		136},
		{TEXT("rt_sigtimedwait"),	137},
		{TEXT("rt_sigqueueinfo"),	138},
		{TEXT("rt_sigreturn"),		139},
		{TEXT("setpriority"),		140},
		{TEXT("getpriority"),		141},
		{TEXT("reboot"),			142},
		{TEXT("setregid"),			143},
		{TEXT("setgid"),			144},
		{TEXT("setreuid"),			145},
		{TEXT("setuid"),			146},
		{TEXT("setresuid"),			147},
		{TEXT("getresuid"),			148},
		{TEXT("setresgid"),			149},
		{TEXT("getresgid"),			150},
		{TEXT("setfsuid"),			151},
		{TEXT("setfsgid"),			152},
		{TEXT("times"),				153},
		{TEXT("setpgid"),			154},
		{TEXT("getpgid"),			155},
		{TEXT("getsid"),			156},
		{TEXT("setsid"),			157},
		{TEXT("getgroups"),			158},
		{TEXT("setgroups"),			159},
		{TEXT("uname"),				160},
		{TEXT("sethostname"),		161},
		{TEXT("setdomainname"),		162},
		{TEXT("getrlimit"),			163},
		{TEXT("setrlimit"),			164},
		{TEXT("getrusage"),			165},
		{TEXT("umask"),				166},
		{TEXT("prctl"),				167},
		{TEXT("getcpu"),			168},
		{TEXT("gettimeofday"),		169},
		{TEXT("settimeofday"),		170},
		{TEXT("adjtimex"),			171},
		{TEXT("getpid"),			172},
		{TEXT("getppid"),			173},
		{TEXT("getuid"),			174},
		{TEXT("geteuid"),			175},
		{TEXT("getgid"),			176},
		{TEXT("getegid"),			177},
		{TEXT("gettid"),			178},
		{TEXT("sysinfo"),			179},
		{TEXT("mq_open"),			180},
		{TEXT("mq_unlink"),			181},
		{TEXT("mq_timedsend"),		182},
		{TEXT("mq_timedreceive"),	183},
		{TEXT("mq_notify"),			184},
		{TEXT("mq_getsetattr"),		185},
		{TEXT("msgget"),			186},
		{TEXT("msgctl"),			187},
		{TEXT("msgrcv"),			188},
		{TEXT("msgsnd"),			189},
		{TEXT("semget"),			190},
		{TEXT("semctl"),			191},
		{TEXT("semtimedop"),		192},
		{TEXT("semop"),				193},
		{TEXT("shmget"),			194},
		{TEXT("shmctl"),			195},
		{TEXT("shmat"),				196},
		{TEXT("shmdt"),				197},
		{TEXT("socket"),			198},
		{TEXT("socketpair"),		199},
		{TEXT("bind"),				200},
		{TEXT("listen"),			201},
		{TEXT("accept"),			202},
		{TEXT("connect"),			203},
		{TEXT("getsockname"),		204},
		{TEXT("getpeername"),		205},
		{TEXT("sendto"),			206},
		{TEXT("recvfrom"),			207},
		{TEXT("setsockopt"),		208},
		{TEXT("getsockopt"),		209},
		{TEXT("shutdown"),			210},
		{TEXT("sendmsg"),			211},
		{TEXT("recvmsg"),			212},
		{TEXT("readahead"),			213},
		{TEXT("brk"),				214},
		{TEXT("munmap"),			215},
		{TEXT("mremap"),			216},
		{TEXT("add_key"),			217},
		{TEXT("request_key"),		218},
		{TEXT("keyctl"),			219},
		{TEXT("clone"),				220},
		{TEXT("execve"),			221},
		{TEXT("mmap"),				222},
		{TEXT("fadvise64"),			223},
		{TEXT("swapon"),			224},
		{TEXT("swapoff"),			225},
		{TEXT("mprotect"),			226},
		{TEXT("msync"),				227},
		{TEXT("mlock"),				228},
		{TEXT("munlock"),			229},
		{TEXT("mlockall"),			230},
		{TEXT("munlockall"),		231},
		{TEXT("mincore"),			232},
		{TEXT("madvise"),			233},
		{TEXT("remap_file_pages"),	234},
		{TEXT("mbind"),				235},
		{TEXT("get_mempolicy"),		236},
		{TEXT("set_mempolicy"),		237},
		{TEXT("migrate_pages"),		238},
		{TEXT("move_pages"),		239},
		{TEXT("rt_tgsigqueueinfo"),	240},
		{TEXT("perf_event_open"),	241},
		{TEXT("accept4"),			242},
		{TEXT("recvmmsg"),			243},
		{TEXT("wait4"),				260},
		{TEXT("prlimit64"),			261},
		{TEXT("fanotify_init"),		262},
		{TEXT("fanotify_mark"),		263},
		{TEXT("name_to_handle_at"),	264},
		{TEXT("open_by_handle_at"),	265},
		{TEXT("clock_adjtime"),		266},
		{TEXT("syncfs"),			267},
		{TEXT("setns"),				268},
		{TEXT("sendmmsg"),			269},
		{TEXT("process_vm_readv"),	270},
		{TEXT("process_vm_writev"),	271},
		{TEXT("kcmp"),				272},
		{TEXT("finit_module"),		273},
		{TEXT("sched_setattr"),		274},
		{TEXT("sched_getattr"),		275},
		{TEXT("renameat2"),			276},
		{TEXT("seccomp"),			277},
		{TEXT("getrandom"),			278},
		{TEXT("memfd_create"),		279},
		{TEXT("bpf"),				280},
		{TEXT("execveat"),			281},
		{TEXT("userfaultfd"),		282},
		{TEXT("membarrier"),		283},
		{TEXT("mlock2"),			284},
		{TEXT("copy_file_range"),	285},
		{TEXT("preadv2"),			286},
		{TEXT("pwritev2"),			287},
		{TEXT("pkey_mprotect"),		288},
		{TEXT("pkey_alloc"),		289},
		{TEXT("pkey_free"),			290},
		{TEXT("statx"),				291},
#else
	#error Unkown Architecture
#endif
	};

	for (int Index = 0; Index < UE_ARRAY_COUNT(SyscallNameToNumber); Index++)
	{
		if (FCString::Strcmp(SyscallNameToNumber[Index].SyscallName, *SyscallName) == 0)
		{
			return SyscallNameToNumber[Index].SyscallNumber;
		}
	}

	return -1;
}
