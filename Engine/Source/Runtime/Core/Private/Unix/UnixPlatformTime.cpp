// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	UnixPlatfomTime.cpp Unix implementations of time functions
=============================================================================*/

#include "Unix/UnixPlatformTime.h"
#include "Containers/StringConv.h"
#include "Logging/LogMacros.h"
#include "GenericPlatform/GenericPlatformTime.h"
#include "CoreGlobals.h"
#include "Containers/Ticker.h"
#include <inttypes.h>
#include <sys/resource.h>

clockid_t FUnixTime::ClockSource = -1;

namespace FUnixTimeInternal
{
	constexpr double TimeValToMicroSec(timeval & tv)
	{
		return static_cast<double>(tv.tv_sec) * 1e6 + static_cast<double>(tv.tv_usec);
	}

	constexpr uint64 TimeSpecToNanoSec(timespec &ts)
	{
		return static_cast<uint64>(static_cast<double>(ts.tv_sec) * 1e9 + static_cast<double>(ts.tv_nsec));
	}

	constexpr double MicroSecondsToSeconds(double MicroSec)
	{
		return MicroSec / 1e6;
	}

	// last time we checked the timer
	static double PreviousUpdateTimeNanoSec = 0.0;

	// last user + system time
	static double PreviousSystemAndUserProcessTimeMicroSec = 0.0;

	// last CPU utilization
	static float CurrentCpuUtilization = 0.0f;
	// last CPU utilization (per core)
	static float CurrentCpuUtilizationNormalized = 0.0f;

	struct FThreadCPUStats
	{
		/** Per-Thread CPU Utilization */
		float ThreadCPUUtilization = 0.f;

		/** Per-Thread CPU Utilization (per Core) */
		float ThreadCPUUtilizationNormalized = 0.f;

		/** The per-thread CPU processing time (kernel + user) from the last update */
		uint64 LastIntervalThreadTimeNS = 0;
	};

	/** Per-Thread CPU Stats */
	thread_local FThreadCPUStats CurrentThreadCPUStats = {};


	/** Process lifetime Soft Page Fault count */
	static uint64 SoftPageFaultCount = 0;

	/** Process lifetime Hard Page Fault count */
	static uint64 HardPageFaultCount = 0;

	/** Process lifetime Blocking Input count */
	static uint64 BlockingInputCount = 0;

	/** Process lifetime Blocking Output count */
	static uint64 BlockingOutputCount = 0;

	/** Process lifetime Voluntary Context Switch count */
	static uint64 VoluntaryContextSwitchCount = 0;

	/** Process lifetime Involuntary Context Switch count */
	static uint64 InvoluntaryContextSwitchCount = 0;

	static char CalibrationLog[4096] = "";

	// Benchmark period is in terms of the clock being benchmarked, so results are not exactly comparable.
	static constexpr uint64 kBenchmarkPeriodNanoSec = 1'000'000'000ULL / 100;	// 0.01s
}

double FUnixTime::InitTiming()
{
	if (ClockSource == -1)
	{
		// Only ever set this ClockSource once
		ClockSource = FUnixTime::CalibrateAndSelectClock();
	}
	SecondsPerCycle = 1e-6;
	SecondsPerCycle64 = 1e-7;

	return FPlatformTime::Seconds();
}

FCPUTime FUnixTime::GetCPUTime()
{
	// 250ms minimum delay between checks to minimize overhead (and also match Windows version)
	constexpr double MinDelayBetweenChecksNanoSec = 250 * 1e6;
	
	struct timespec ts;
	if (0 == clock_gettime(GetClockSource(), &ts))
	{
		const double CurrentTimeNanoSec = static_cast<double>(FUnixTimeInternal::TimeSpecToNanoSec(ts));

		// see if we need to update the values
		double TimeSinceLastUpdateNanoSec = CurrentTimeNanoSec - FUnixTimeInternal::PreviousUpdateTimeNanoSec;
		if (TimeSinceLastUpdateNanoSec >= MinDelayBetweenChecksNanoSec)
		{
			const float DeltaTimeInMs = MinDelayBetweenChecksNanoSec / 1e6;
			UpdateCPUTime(DeltaTimeInMs);
			FUnixTimeInternal::PreviousUpdateTimeNanoSec = CurrentTimeNanoSec;
		}
	}

	return FCPUTime(FUnixTimeInternal::CurrentCpuUtilizationNormalized, FUnixTimeInternal::CurrentCpuUtilization);
}

FCPUTime FUnixTime::GetThreadCPUTime()
{
	return FCPUTime(FUnixTimeInternal::CurrentThreadCPUStats.ThreadCPUUtilizationNormalized,
					FUnixTimeInternal::CurrentThreadCPUStats.ThreadCPUUtilization);
}

bool FUnixTime::UpdateCPUTime(float InDeltaTimeInMs)
{
	rusage Usage;

	double DeltaTimeInMs = InDeltaTimeInMs;

	if (getrusage(RUSAGE_SELF, &Usage) == 0)
	{
		// Get delta between last two calls if the passed DeltaTime is zero
		if (DeltaTimeInMs <= 0.0)
		{
			timespec ts;

			if (clock_gettime(ClockSource, &ts) == 0)
			{
				const double CurrentTimeNanoSec = static_cast<double>(FUnixTimeInternal::TimeSpecToNanoSec(ts));

				DeltaTimeInMs = (CurrentTimeNanoSec - FUnixTimeInternal::PreviousUpdateTimeNanoSec) / 1e6;
				FUnixTimeInternal::PreviousUpdateTimeNanoSec = CurrentTimeNanoSec;
			}
		}

		const double DeltaTimeInMicroSec = DeltaTimeInMs * 1e3;
		const double CurrentSystemAndUserProcessTimeMicroSec = FUnixTimeInternal::TimeValToMicroSec(Usage.ru_utime) + FUnixTimeInternal::TimeValToMicroSec(Usage.ru_stime); // holds all usages on all cores
		const double CpuTimeDuringPeriodMicroSec = CurrentSystemAndUserProcessTimeMicroSec - FUnixTimeInternal::PreviousSystemAndUserProcessTimeMicroSec;

		double CurrentCpuUtilizationHighPrec = (CpuTimeDuringPeriodMicroSec / DeltaTimeInMicroSec) * 100.0;

		// recalculate the values
		FUnixTimeInternal::CurrentCpuUtilizationNormalized = static_cast<float>(CurrentCpuUtilizationHighPrec / static_cast<double>(FPlatformMisc::NumberOfCoresIncludingHyperthreads()));
		FUnixTimeInternal::CurrentCpuUtilization = static_cast<float>(CurrentCpuUtilizationHighPrec);

		// update previous
		FUnixTimeInternal::PreviousSystemAndUserProcessTimeMicroSec = CurrentSystemAndUserProcessTimeMicroSec;
		
		LastIntervalCPUTimeInSeconds = FUnixTimeInternal::MicroSecondsToSeconds(CpuTimeDuringPeriodMicroSec);


		// Free performance stats
		FUnixTimeInternal::SoftPageFaultCount = Usage.ru_minflt;
		FUnixTimeInternal::HardPageFaultCount = Usage.ru_majflt;
		FUnixTimeInternal::BlockingInputCount = Usage.ru_inblock;
		FUnixTimeInternal::BlockingOutputCount = Usage.ru_oublock;
		FUnixTimeInternal::VoluntaryContextSwitchCount = Usage.ru_nvcsw;
		FUnixTimeInternal::InvoluntaryContextSwitchCount = Usage.ru_nivcsw;
	}

	return true;
}

bool FUnixTime::UpdateThreadCPUTime(float/*= 0.0*/)
{
	bool bReturnVal = false;

#ifdef PLATFORM_HAS_BSD_THREAD_CPUTIME
	timespec SystemTime;
	timespec ThreadTime;

	if (clock_gettime(ClockSource, &SystemTime) == 0 && clock_gettime(CLOCK_THREAD_CPUTIME_ID, &ThreadTime) == 0)
	{
		struct FThreadCPUTime
		{
			uint64 LastThreadCPUTimeUpdateNS = 0;
			uint64 LastThreadTimeNS = 0;
		};

		thread_local FThreadCPUTime ThreadTimeInfo = {};

		const uint64 SystemTimeNS = FUnixTimeInternal::TimeSpecToNanoSec(SystemTime);
		const uint64 ThreadTimeNS = FUnixTimeInternal::TimeSpecToNanoSec(ThreadTime);
		const uint64 DeltaTimeNS = SystemTimeNS - ThreadTimeInfo.LastThreadCPUTimeUpdateNS;

		ThreadTimeInfo.LastThreadCPUTimeUpdateNS = SystemTimeNS;

		const uint64 ElapsedThreadCPUTimeNS = ThreadTimeNS - ThreadTimeInfo.LastThreadTimeNS;
		const double ThreadCPUUtilizationHighPrec = (static_cast<double>(ElapsedThreadCPUTimeNS) / static_cast<double>(DeltaTimeNS)) * 100.0;

		FUnixTimeInternal::CurrentThreadCPUStats.ThreadCPUUtilization = static_cast<float>(ThreadCPUUtilizationHighPrec);
		FUnixTimeInternal::CurrentThreadCPUStats.ThreadCPUUtilizationNormalized =
																static_cast<float>(ThreadCPUUtilizationHighPrec / FPlatformMisc::NumberOfCoresIncludingHyperthreads());

		ThreadTimeInfo.LastThreadTimeNS = ThreadTimeNS;
		FUnixTimeInternal::CurrentThreadCPUStats.LastIntervalThreadTimeNS = ElapsedThreadCPUTimeNS;

		bReturnVal = true;
	}
#endif

	return bReturnVal;
}

void FUnixTime::AutoUpdateGameThreadCPUTime(double UpdateInterval)
{
	static bool bEnabledGameThreadTiming = false;

	if (!bEnabledGameThreadTiming)
	{
		FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateStatic(&FPlatformTime::UpdateThreadCPUTime), (float)UpdateInterval);

		bEnabledGameThreadTiming = true;
	}
}

double FUnixTime::GetLastIntervalThreadCPUTimeInSeconds()
{
	return static_cast<double>(FUnixTimeInternal::CurrentThreadCPUStats.LastIntervalThreadTimeNS) / 1e9;
}

// Benchmark a clock.
// Returns the number of times this clock can be called per second, or zero if the clock is totally unusable.
[[nodiscard]]
static uint64 BenchmarkClock(const clockid_t Id, const char* Name)
{
	using namespace FUnixTimeInternal;

	char Buffer[256] = "";

	struct timespec ts;
	if (clock_gettime(Id, &ts) == -1)
	{
		FCStringAnsi::Snprintf(Buffer, sizeof(Buffer), "Clock_id %d (%s) is not supported on this system, clock_gettime() fails.\n", Id, Name);
		FCStringAnsi::StrncatTruncateDest(CalibrationLog, sizeof(CalibrationLog), Buffer);
		return 0;
	}

	// clock_getres() can fail when running on Windows Subsystem for Linux version 1 (but the clock can still be supported).
	struct timespec Resolution;
	if (clock_getres(Id, &Resolution) == -1)
	{
		Resolution = {-1, -1};
	}

	// from now on we'll assume that clock_gettime cannot fail
	uint64 StartTimeNS = TimeSpecToNanoSec(ts);
	uint64 EndTimeNS = StartTimeNS;

	uint64 NumCalls = 1;	// account for starting timestamp
	uint64 NumZeroDeltas = 0;
	const uint64 kHardLimitOnZeroDeltas = (1 << 26);	// arbitrary, but high enough so we don't hit it on fast coarse clocks
	do
	{
		clock_gettime(Id, &ts);

		uint64 NewEndTimeNS = TimeSpecToNanoSec(ts);
		++NumCalls;

		if (NewEndTimeNS < EndTimeNS)
		{
			FCStringAnsi::Snprintf(Buffer, sizeof(Buffer), "Clock_id %d (%s) is not monotonic.\n", Id, Name);
			FCStringAnsi::StrncatTruncateDest(CalibrationLog, sizeof(CalibrationLog), Buffer);
			return NumCalls;
		}

		if (NewEndTimeNS == EndTimeNS)
		{
			++NumZeroDeltas;

			// do not lock up if the clock is broken (e.g. stays in place)
			if (NumZeroDeltas > kHardLimitOnZeroDeltas)
			{
				FCStringAnsi::Snprintf(Buffer, sizeof(Buffer), "Clock_id %d (%s) has many (%llu) zero deltas.\n", Id, Name, NumZeroDeltas);
				FCStringAnsi::StrncatTruncateDest(CalibrationLog, sizeof(CalibrationLog), Buffer);
				return NumCalls;
			}
		}

		EndTimeNS = NewEndTimeNS;
	}
	while (EndTimeNS - StartTimeNS < kBenchmarkPeriodNanoSec);

	const double BenchmarksPerSecond = 1e9 / static_cast<double>(EndTimeNS - StartTimeNS);
	uint64 RealNumCallsPerSecond = static_cast<uint64>(BenchmarksPerSecond * static_cast<double>(NumCalls));

	char ZeroDeltasBuf[128] = "";
	if (NumZeroDeltas)
	{
		FCStringAnsi::Snprintf(ZeroDeltasBuf, sizeof(ZeroDeltasBuf), "with %f%% zero deltas", 100.0 * static_cast<double>(NumZeroDeltas) / static_cast<double>(NumCalls));
	}

	FCStringAnsi::Snprintf(Buffer, sizeof(Buffer), (" - %s (id=%d), resolution %" PRId64 ".%.09" PRId32 "s, can sustain %" PRIu64 " (%" PRIu64 "K, % " PRIu64 "M) calls per second %s.\n"),
		Name, static_cast<int>(Id),
		static_cast<int64>(Resolution.tv_sec), static_cast<int32>(Resolution.tv_nsec),
		RealNumCallsPerSecond, (RealNumCallsPerSecond + 500) / 1'000, (RealNumCallsPerSecond + 500'000) / 1'000'000,
		NumZeroDeltas ? ZeroDeltasBuf : "without zero deltas");
	FCStringAnsi::StrncatTruncateDest(CalibrationLog, sizeof(CalibrationLog), Buffer);

	return RealNumCallsPerSecond;
}

clockid_t FUnixTime::CalibrateAndSelectClock()
{
	using namespace FUnixTimeInternal;

	// do not calibrate in case of programs, so e.g. ShaderCompileWorker speed is not impacted
	if constexpr (IS_PROGRAM)
	{
		struct timespec ts;
		if (clock_gettime(CLOCK_MONOTONIC, &ts) == -1)
		{
			FCStringAnsi::Snprintf(CalibrationLog, sizeof(CalibrationLog),
				"Skipped benchmarking clocks because the engine is running in a standalone program mode: CLOCK_MONOTONIC is unavailable, CLOCK_REALTIME will be used.\n");
			return CLOCK_REALTIME;
		}

		FCStringAnsi::Snprintf(CalibrationLog, sizeof(CalibrationLog),
			"Skipped benchmarking clocks because the engine is running in a standalone program mode - CLOCK_MONOTONIC will be used.\n");
		return CLOCK_MONOTONIC;
	}

	char Buffer[256];

	// init calibration log
	FCStringAnsi::Snprintf(CalibrationLog, sizeof(CalibrationLog), "Benchmarking clocks:\n");

	struct FClockDesc
	{
		clockid_t Id = -1;
		const char* Name = "";
		// Higher values are more preferred.
		int Preference = 0;
		uint64 CallsPerSecond = 0;
	}
	Clocks[] =
	{
		{ CLOCK_MONOTONIC, "CLOCK_MONOTONIC", 100 }, // Highest preference
		{ CLOCK_MONOTONIC_COARSE, "CLOCK_MONOTONIC_COARSE", 20 },
		{ CLOCK_MONOTONIC_RAW, "CLOCK_MONOTONIC_RAW", 10 },
		{ CLOCK_REALTIME, "CLOCK_REALTIME", 1 }, // Last resort.
	};

	for (FClockDesc& Clock : Clocks)
	{
		Clock.CallsPerSecond = BenchmarkClock(Clock.Id, Clock.Name);
		if (Clock.CallsPerSecond == 0)
		{
			Clock.Preference = 0;
		}
	}

	// Select preferred clock
	int ChosenClockIdx = 0;
	for (int Idx = 0; Idx < UE_ARRAY_COUNT(Clocks); ++Idx)
	{
		if (Clocks[Idx].Preference > Clocks[ChosenClockIdx].Preference)
		{
			ChosenClockIdx = Idx;
		}
	}

	FCStringAnsi::Snprintf(Buffer, sizeof(Buffer), "Selected %s (%d) as the best available clock.\n", Clocks[ChosenClockIdx].Name, Clocks[ChosenClockIdx].Id);
	FCStringAnsi::StrncatTruncateDest(CalibrationLog, sizeof(CalibrationLog), Buffer);

	// Warn if our current clock source cannot be called at least 1M times a second (<30k a frame) as this may affect tight loops
	if (Clocks[ChosenClockIdx].CallsPerSecond < 1'000'000)
	{
		FCStringAnsi::Snprintf(Buffer, sizeof(Buffer), "The clock source (%s) is too slow on this machine, performance may be affected.\n", Clocks[ChosenClockIdx].Name);
		FCStringAnsi::StrncatTruncateDest(CalibrationLog, sizeof(CalibrationLog), Buffer);
	}

	return Clocks[ChosenClockIdx].Id;
}

void FUnixTime::PrintCalibrationLog()
{
	// clock selection happens too early to be printed to log, print it now
	FString Buffer(ANSI_TO_TCHAR(FUnixTimeInternal::CalibrationLog));

	TArray<FString> Lines;
	Buffer.ParseIntoArrayLines(Lines);

	for(const FString& Line : Lines)
	{
		UE_LOGF(LogCore, Log, "%ls", *Line);
	}
}

bool FUnixPlatformMisc::GetPageFaultStats(FPageFaultStats& OutStats, EPageFaultFlags Flags/*=EPageFaultFlags::All*/)
{
	// Ignore flags since all stats are free
	OutStats.SoftPageFaults = FUnixTimeInternal::SoftPageFaultCount;
	OutStats.HardPageFaults = FUnixTimeInternal::HardPageFaultCount;
	OutStats.TotalPageFaults = FUnixTimeInternal::SoftPageFaultCount + FUnixTimeInternal::HardPageFaultCount;

	return true;
}

bool FUnixPlatformMisc::GetBlockingIOStats(FProcessIOStats& OutStats, EInputOutputFlags Flags/*=EInputOutputFlags::All*/)
{
	bool bSuccess = false;

	if (EnumHasAnyFlags(Flags, EInputOutputFlags::BlockingInput | EInputOutputFlags::BlockingOutput))
	{
		OutStats.BlockingInput = FUnixTimeInternal::BlockingInputCount;
		OutStats.BlockingOutput = FUnixTimeInternal::BlockingOutputCount;

		bSuccess = true;
	}

	return bSuccess;
}

bool FUnixPlatformMisc::GetContextSwitchStats(FContextSwitchStats& OutStats, EContextSwitchFlags Flags/*=EContextSwitchFlags::All*/)
{
	// Ignore flags since all stats are free
	OutStats.VoluntaryContextSwitches = FUnixTimeInternal::VoluntaryContextSwitchCount;
	OutStats.InvoluntaryContextSwitches = FUnixTimeInternal::InvoluntaryContextSwitchCount;
	OutStats.TotalContextSwitches = FUnixTimeInternal::VoluntaryContextSwitchCount + FUnixTimeInternal::InvoluntaryContextSwitchCount;

	return true;
}

