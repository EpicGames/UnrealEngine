// Copyright Epic Games, Inc. All Rights Reserved.

#include "Unix/UnixSignalHeartBeat.h"
#include "Unix/UnixPlatformRealTimeSignals.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Logging/LogMacros.h"
#include "HAL/PlatformTime.h"
#include "AutoRTFM.h"

#include <signal.h>
#include <unistd.h>

DEFINE_LOG_CATEGORY_STATIC(LogUnixHeartBeat, Log, All);

FUnixSignalGameHitchHeartBeat* FUnixSignalGameHitchHeartBeat::Singleton = nullptr;

FUnixSignalGameHitchHeartBeat& FUnixSignalGameHitchHeartBeat::Get()
{
	struct FInitHelper
	{
		FUnixSignalGameHitchHeartBeat* Instance;

		FInitHelper()
		{
			check(!Singleton);
			Instance = new FUnixSignalGameHitchHeartBeat;
			Singleton = Instance;
		}

		~FInitHelper()
		{
			Singleton = nullptr;

			delete Instance;
			Instance = nullptr;
		}
	};

	// Use a function static helper to ensure creation
	// of the FUnixSignalGameHitchHeartBeat instance is thread safe.
	static FInitHelper Helper;
	return *Helper.Instance;
}

FUnixSignalGameHitchHeartBeat* FUnixSignalGameHitchHeartBeat::GetNoInit()
{
	return Singleton;
}

namespace
{
	// 1ms for the lowest amount allowed for hitch detection. Anything less we wont try to detect hitches
	double MinimalHitchThreashold = 0.001;

	void SignalHitchHandler(int Signal, siginfo_t* info, void* context)
	{
#if USE_HITCH_DETECTION
		UE_LOGF(LogUnixHeartBeat, Verbose, "SignalHitchHandler");
		GHitchDetected = true;
#endif
	}

	timespec TimerGetTime(timer_t TimerId)
	{
		struct itimerspec HeartBeatTime;
		FMemory::Memzero(HeartBeatTime);

		if (timer_gettime(TimerId, &HeartBeatTime) == -1)
		{
			int Errno = errno;
			UE_LOGF(LogUnixHeartBeat, Warning, "Failed to timer_gettime() errno=%d (%ls)", Errno, UTF8_TO_TCHAR(strerror(Errno)));
		}

		return HeartBeatTime.it_value;
	}
}

FUnixSignalGameHitchHeartBeat::FUnixSignalGameHitchHeartBeat()
{
	Init();
}

FUnixSignalGameHitchHeartBeat::~FUnixSignalGameHitchHeartBeat()
{
	if (bTimerCreated)
	{
		timer_delete(TimerId);
		bTimerCreated = false;
		TimerId = 0;
	}
}

void FUnixSignalGameHitchHeartBeat::Init()
{
#if USE_HITCH_DETECTION
	// Setup the signal callback for when we hit a hitch
	struct sigaction SigAction;
	FMemory::Memzero(SigAction);
	SigAction.sa_flags = SA_SIGINFO;
	SigAction.sa_sigaction = SignalHitchHandler;

	const bool bActionCreated = sigaction(HEART_BEAT_SIGNAL, &SigAction, nullptr) == 0;
	if (!bActionCreated)
	{
		const int Errno = errno;
		UE_LOGF(LogUnixHeartBeat, Warning, "Failed to sigaction() errno=%d (%ls)", Errno, UTF8_TO_TCHAR(strerror(Errno)));
	}

	struct sigevent SignalEvent;
	FMemory::Memzero(SignalEvent);
	SignalEvent.sigev_notify = SIGEV_SIGNAL;
	SignalEvent.sigev_signo = HEART_BEAT_SIGNAL;

	bTimerCreated = timer_create(CLOCK_REALTIME, &SignalEvent, &TimerId) == 0;
	if (!bTimerCreated)
	{
		const int Errno = errno;
		UE_LOGF(LogUnixHeartBeat, Warning, "Failed to timer_create() errno=%d (%ls)", Errno, UTF8_TO_TCHAR(strerror(Errno)));
	}

	float CmdLine_HitchDurationS = 0.0;
	bHasCmdLine = FParse::Value(FCommandLine::Get(), TEXT("hitchdetection="), CmdLine_HitchDurationS);

	if (bHasCmdLine)
	{
		HitchThresholdS = CmdLine_HitchDurationS;
	}

	SuspendCount = 0;

	InitSettings();
#endif
}

void FUnixSignalGameHitchHeartBeat::InitSettings()
{
	static bool bFirst = true;

	// Command line takes priority over config, so only check the ini if we didnt already set our selfs from the cmd line
	if (!bHasCmdLine)
	{
		float Config_HitchDurationS = 0.0;
		bool bReadFromConfig = GConfig->GetFloat(TEXT("Core.System"), TEXT("GameThreadHeartBeatHitchDuration"), Config_HitchDurationS, GEngineIni);

		if (bReadFromConfig)
		{
			HitchThresholdS = Config_HitchDurationS;
		}
	}

	if (bFirst)
	{
		bFirst = false;

		bStartSuspended = false;
		GConfig->GetBool(TEXT("Core.System"), TEXT("GameThreadHeartBeatStartSuspended"), bStartSuspended, GEngineIni);

		if (FParse::Param(FCommandLine::Get(), TEXT("hitchdetectionstartsuspended")))
		{
			bStartSuspended = true;
		}
		else if (FParse::Param(FCommandLine::Get(), TEXT("hitchdetectionstartrunning")))
		{
			bStartSuspended = false;
		}

		if (bStartSuspended)
		{
			SuspendCount = 1;
		}
	}

	UE_LOGF(LogUnixHeartBeat, Verbose, "HitchThresholdS:%f", HitchThresholdS);
}

void FUnixSignalGameHitchHeartBeat::FrameStart(bool bSkipThisFrame)
{
#if USE_HITCH_DETECTION
	check(IsInGameThread());

	UE_LOGF(LogUnixHeartBeat, VeryVerbose, "bDisabled:%ls SuspendCount:%d bTimerCreated:%ls TimerId:%d timer:%d",
		bDisabled ? TEXT("true") : TEXT("false"), 
		SuspendCount, 
		bTimerCreated ? TEXT("true") : TEXT("false"),
		TimerId, 
		bTimerCreated ? TimerGetTime(TimerId).tv_nsec : 0);

	if (!bDisabled && SuspendCount == 0 && bTimerCreated)
	{
		UE_LOGF(LogUnixHeartBeat, VeryVerbose, "HitchThresholdS:%f MinimalHitchThreashold:%f",
			HitchThresholdS, MinimalHitchThreashold);

		if (HitchThresholdS > MinimalHitchThreashold)
		{
			struct itimerspec HeartBeatTime;
			FMemory::Memzero(HeartBeatTime);

			long FullSeconds = static_cast<long>(HitchThresholdS);
			long RemainderInNanoSeconds = static_cast<long>(FMath::Fmod(HitchThresholdS, 1.0) * 1000000000.0);
			HeartBeatTime.it_value.tv_sec = FullSeconds;
			HeartBeatTime.it_value.tv_nsec = RemainderInNanoSeconds;

			if (GHitchDetected)
			{
				UE_LOGF(LogCore, Error, "Hitch detected on previous gamethread frame (%8.2fms since last frame)", float(FPlatformTime::Seconds() - StartTime) * 1000.0f);
			}

			StartTime = FPlatformTime::Seconds();

			if (timer_settime(TimerId, 0, &HeartBeatTime, nullptr) == -1)
			{
				int Errno = errno;
				UE_LOGF(LogUnixHeartBeat, Warning, "Failed to timer_settime() errno=%d (%ls)", Errno, UTF8_TO_TCHAR(strerror(Errno)));
			}
		}
	}

	GHitchDetected = false;
#endif
}

// If the process is suspended we will hit a hitch for how ever long it was suspended for
double FUnixSignalGameHitchHeartBeat::GetFrameStartTime()
{
	return StartTime;
}

double FUnixSignalGameHitchHeartBeat::GetCurrentTime()
{
	return FPlatformTime::Seconds();
}

void FUnixSignalGameHitchHeartBeat::SuspendHeartBeat()
{
#if USE_HITCH_DETECTION
	if (!IsInGameThread())
	{
		return;
	}

	SuspendCount++;
	UE_LOGF(LogUnixHeartBeat, Verbose, "SuspendCount:%d", SuspendCount);

	if (bTimerCreated)
	{
		UE_AUTORTFM_OPEN
		{
			struct itimerspec DisarmTime;
			FMemory::Memzero(DisarmTime);

			if (timer_settime(TimerId, 0, &DisarmTime, nullptr) == -1)
			{
				int Errno = errno;
				UE_LOGF(LogUnixHeartBeat, Warning, "Failed to timer_settime() errno=%d (%ls)", Errno, UTF8_TO_TCHAR(strerror(Errno)));
			}
		};

		UE_AUTORTFM_ONABORT(this)
		{
			ResumeHeartBeat();
		};
	}
#endif
}

void FUnixSignalGameHitchHeartBeat::ResumeHeartBeat()
{
#if USE_HITCH_DETECTION
	if (!IsInGameThread())
	{
		return;
	}

	UE_AUTORTFM_ONCOMMIT(this)
	{
		if (SuspendCount > 0)
		{
			SuspendCount--;
			UE_LOGF(LogUnixHeartBeat, Verbose, "SuspendCount:%d", SuspendCount);

			FrameStart(true);
		}
	};
#endif
}

bool FUnixSignalGameHitchHeartBeat::IsStartedSuspended()
{
	return bStartSuspended;
}

void FUnixSignalGameHitchHeartBeat::Restart()
{
	UE_LOGF(LogUnixHeartBeat, Verbose, "Restart");
	bDisabled = false;

	// If we still have a valid handle on the timer_t clean it up
	if (bTimerCreated)
	{
		timer_delete(TimerId);
		bTimerCreated = false;
		TimerId = 0;
	}

	Init();
}

void FUnixSignalGameHitchHeartBeat::Stop()
{
	UE_LOGF(LogUnixHeartBeat, Verbose, "Stop");
	SuspendHeartBeat();
	bDisabled = true;
}

void FUnixSignalGameHitchHeartBeat::PostFork()
{
	UE_LOGF(LogUnixHeartBeat, Verbose, "PostFork");
	// timers aren't inherited by child processes
	bTimerCreated = false;
	TimerId = 0;
}
