// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utf8/Utf8Iterator.h"

#include "Containers/StringView.h"

// ---------------------------------------------------------------------------
// Internal helpers
// ---------------------------------------------------------------------------

namespace
{
	/** Returns true when Octet has the bit pattern 10xxxxxx (a UTF-8 continuation byte). */
	UE_REWRITE bool IsContinuationByte(uint32 Octet)
	{
		return (Octet & 0xC0u) == 0x80u;
	}

	/** Returns true when Codepoint is in the UTF-16 surrogate range [U+D800, U+DFFF]. */
	UE_REWRITE bool IsSurrogate(uint32 Codepoint)
	{
		return (Codepoint & 0xFFFFF800u) == 0xD800u;
	}
} // anonymous namespace

// ---------------------------------------------------------------------------
// FUtf8Iterator
// ---------------------------------------------------------------------------

FUtf8Iterator::FUtf8Iterator(FUtf8StringView InView)
	: Pos(InView.GetData())
	, ViewEnd(InView.GetData() + InView.Len())
	, CachedCodepoint(ReplacementCharacter)
{
	if (Pos != ViewEnd)
	{
		DecodeNext();
	}
	else
	{
		ViewEnd = nullptr;
	}
}

void FUtf8Iterator::operator++()
{
	checkfSlow(*this, TEXT("Iterator has already reached the end"));
	if (Pos != ViewEnd)
	{
		DecodeNext();
	}
	else
	{
		ViewEnd = nullptr;
	}
}

void FUtf8Iterator::DecodeNext()
{
	// Precondition: Pos < ViewEnd
	const uint32 Lead = (uint32)(uint8)*Pos;

	// -----------------------------------------------------------------------
	// Single-byte sequence (U+0000 – U+007F)
	// -----------------------------------------------------------------------
	if (Lead < 0x80u)
	{
		CachedCodepoint = (UTF32CHAR)Lead;
		++Pos;
		return;
	}

	// -----------------------------------------------------------------------
	// Continuation byte in lead position, or 0xC0/0xC1 which can never start
	// a valid sequence (all their encodings would be overlong) — consume just
	// this byte.
	// -----------------------------------------------------------------------
	if (Lead < 0xC2u)
	{
		CachedCodepoint = ReplacementCharacter;
		++Pos;
		return;
	}

	// -----------------------------------------------------------------------
	// Two-byte sequence (U+0080 – U+07FF)
	// -----------------------------------------------------------------------
	if (Lead < 0xE0u)
	{
		if (ViewEnd - Pos < 2)
		{
			// Truncated at end of view
			CachedCodepoint = ReplacementCharacter;
			Pos = ViewEnd;
			return;
		}

		const uint32 Cont1 = (uint32)(uint8)Pos[1];
		if (!IsContinuationByte(Cont1))
		{
			// Not a continuation byte; do not consume it
			CachedCodepoint = ReplacementCharacter;
			Pos += 1;
			return;
		}

		const uint32 Codepoint = ((Lead & 0x1Fu) << 6u) | (Cont1 & 0x3Fu);
		// For Lead >= 0xC2 the minimum codepoint is U+0080, so the overlong check
		// below is unreachable; it is kept as a safety net.
		CachedCodepoint = (Codepoint < 0x80u) ? ReplacementCharacter : (UTF32CHAR)Codepoint;
		Pos += 2;
		return;
	}

	// -----------------------------------------------------------------------
	// Three-byte sequence (U+0800 – U+FFFF, excluding surrogates)
	// -----------------------------------------------------------------------
	if (Lead < 0xF0u)
	{
		if (ViewEnd - Pos < 2)
		{
			CachedCodepoint = ReplacementCharacter;
			Pos = ViewEnd;
			return;
		}

		const uint32 Cont1 = (uint32)(uint8)Pos[1];
		if (!IsContinuationByte(Cont1))
		{
			CachedCodepoint = ReplacementCharacter;
			Pos += 1;
			return;
		}

		// Structural range check on the first continuation byte:
		//   0xE0 requires Cont1 in 0xA0–0xBF to avoid overlong sequences.
		//   0xED requires Cont1 in 0x80–0x9F to avoid surrogate codepoints.
		// When the check fails, consume only the lead byte; Cont1 will be decoded
		// as a bare continuation (→ replacement) on the next step.
		if ((Lead == 0xE0u && Cont1 < 0xA0u) || (Lead == 0xEDu && Cont1 >= 0xA0u))
		{
			CachedCodepoint = ReplacementCharacter;
			Pos += 1;
			return;
		}

		if (ViewEnd - Pos < 3)
		{
			CachedCodepoint = ReplacementCharacter;
			Pos = ViewEnd;
			return;
		}

		const uint32 Cont2 = (uint32)(uint8)Pos[2];
		if (!IsContinuationByte(Cont2))
		{
			CachedCodepoint = ReplacementCharacter;
			Pos += 2;
			return;
		}

		const uint32 Codepoint = ((Lead & 0x0Fu) << 12u) | ((Cont1 & 0x3Fu) << 6u) | (Cont2 & 0x3Fu);
		// The structural range checks above make the overlong and surrogate cases
		// unreachable; they are kept as a safety net.
		CachedCodepoint = (Codepoint < 0x800u || IsSurrogate(Codepoint)) ? ReplacementCharacter : (UTF32CHAR)Codepoint;
		Pos += 3;
		return;
	}

	// -----------------------------------------------------------------------
	// Four-byte sequence (U+10000 – U+10FFFF)
	// -----------------------------------------------------------------------
	if (Lead < 0xF8u)
	{
		// 0xF5–0xF7 always produce codepoints above U+10FFFF; consume only the
		// lead byte so that any following bytes are decoded independently.
		if (Lead >= 0xF5u)
		{
			CachedCodepoint = ReplacementCharacter;
			++Pos;
			return;
		}

		if (ViewEnd - Pos < 2)
		{
			CachedCodepoint = ReplacementCharacter;
			Pos = ViewEnd;
			return;
		}

		const uint32 Cont1 = (uint32)(uint8)Pos[1];
		if (!IsContinuationByte(Cont1))
		{
			CachedCodepoint = ReplacementCharacter;
			Pos += 1;
			return;
		}

		// Structural range check on the first continuation byte:
		//   0xF0 requires Cont1 in 0x90–0xBF to avoid overlong sequences.
		//   0xF4 requires Cont1 in 0x80–0x8F to avoid codepoints above U+10FFFF.
		// When the check fails, consume only the lead byte.
		if ((Lead == 0xF0u && Cont1 < 0x90u) || (Lead == 0xF4u && Cont1 >= 0x90u))
		{
			CachedCodepoint = ReplacementCharacter;
			Pos += 1;
			return;
		}

		if (ViewEnd - Pos < 3)
		{
			CachedCodepoint = ReplacementCharacter;
			Pos = ViewEnd;
			return;
		}

		const uint32 Cont2 = (uint32)(uint8)Pos[2];
		if (!IsContinuationByte(Cont2))
		{
			CachedCodepoint = ReplacementCharacter;
			Pos += 2;
			return;
		}

		if (ViewEnd - Pos < 4)
		{
			CachedCodepoint = ReplacementCharacter;
			Pos = ViewEnd;
			return;
		}

		const uint32 Cont3 = (uint32)(uint8)Pos[3];
		if (!IsContinuationByte(Cont3))
		{
			CachedCodepoint = ReplacementCharacter;
			Pos += 3;
			return;
		}

		const uint32 Codepoint = ((Lead & 0x07u) << 18u) | ((Cont1 & 0x3Fu) << 12u) | ((Cont2 & 0x3Fu) << 6u) | (Cont3 & 0x3Fu);
		// The structural range checks above make the overlong and out-of-range cases
		// unreachable; they are kept as a safety net.
		CachedCodepoint = (Codepoint < 0x10000u || Codepoint > 0x10FFFFu) ? ReplacementCharacter : (UTF32CHAR)Codepoint;
		Pos += 4;
		return;
	}

	// -----------------------------------------------------------------------
	// Five-byte and six-byte sequences — illegal per RFC 3629.
	// Also catches 0xFE and 0xFF which are never valid UTF-8 bytes.
	// Consume only the lead byte; any following bytes are decoded independently.
	// -----------------------------------------------------------------------
	CachedCodepoint = ReplacementCharacter;
	++Pos;
}
