// Copyright Epic Games, Inc. All Rights Reserved.

#include "Virtualization/VirtualizationSystem.h"

#include "CoreGlobals.h"
#include "HAL/IConsoleManager.h"
#include "Misc/App.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/ScopeLock.h"

// Can be defined as 1 by programs target.cs files to disable the virtualization system
// entirely. This could be removed in the future if we move the virtualization module
// to be a plugin
#ifndef UE_DISABLE_VIRTUALIZATION_SYSTEM
	#define UE_DISABLE_VIRTUALIZATION_SYSTEM 0
#endif //UE_DISABLE_VIRTUALIZATION_SYSTEM

// Can be defined as 1 by programs target.cs files to change the initialization of
// the virtualization system to be a lazy initialization.
#ifndef UE_VIRTUALIZATION_SYSTEM_LAZY_INIT
	#define UE_VIRTUALIZATION_SYSTEM_LAZY_INIT 0
#else
	#pragma message("WARNING: Use of UE_VIRTUALIZATION_SYSTEM_LAZY_INIT is deprecated as of UE 5.5, consider replacing with 'ini:Engine:[Core.VirtualizationModule]:LazyInitConnections=true'")
#endif //UE_VIRTUALIZATION_SYSTEM_LAZY_INIT

// When enabled we will log if FNullVirtualizationSystem tries to push or pull payloads
#define UE_LOG_ON_NULLSYSTEM_USE 1

// When enabled messages logged when FNullVirtualizationSystem is in use will use the 'Error'
// severity. When disabled the messages will use 'Warning' severity.
#define UE_USING_NULLSYSTEM_IS_ERROR 1

#if UE_USING_NULLSYSTEM_IS_ERROR
	#define UE_NULLSYSTEM_SEVERITY Error
#else
	#define UE_NULLSYSTEM_SEVERITY Warning
#endif // UE_USING_NULLSYSTEM_IS_ERROR

namespace UE::Virtualization
{

static TAutoConsoleVariable<bool> CVarDisableSystem(
	TEXT("VA.DisableSystem"),
	false,
	TEXT("When true the VA system will be disabled as though 'SystemName' was 'None'"));

static TAutoConsoleVariable<bool> CVarLazyInitSystem(
	TEXT("VA.LazyInitSystem"),
	false,
	TEXT("When true the VA system will be lazy initialized on first use"));

/** Default implementation to be used when the system is disabled */
class FNullVirtualizationSystem : public IVirtualizationSystem
{
public:
	FNullVirtualizationSystem()
	{
		UE_LOGF(LogVirtualization, Display, "FNullVirtualizationSystem mounted, virtualization will be disabled");
	}

	virtual ~FNullVirtualizationSystem() = default;

	virtual bool Initialize(const FInitParams& InitParams) override
	{
		return true;
	}

	virtual bool IsEnabled() const override
	{
		return false;
	}

	virtual bool IsPushingEnabled(EStorageType StorageType) const override
	{
		return false;
	}

	virtual EPayloadFilterReason FilterPayload(const UObject* Owner) const override
	{
		return EPayloadFilterReason::None;
	}

	virtual bool AllowSubmitIfVirtualizationFailed() const override
	{
		return false;
	}

	virtual bool PushData(TArrayView<FPushRequest> Requests, EStorageType StorageType) override
	{
#if UE_LOG_ON_NULLSYSTEM_USE
		UE_LOGF(LogVirtualization, UE_NULLSYSTEM_SEVERITY, "Cannot push payloads as the virtualization system is disabled");
#endif //UE_LOG_ON_NULLSYSTEM_USE

		return false;
	}

	virtual bool PullData(TArrayView<FPullRequest> Requests) override
	{
#if UE_LOG_ON_NULLSYSTEM_USE
		for (const FPullRequest& Request : Requests)
		{
			UE_LOGF(LogVirtualization, UE_NULLSYSTEM_SEVERITY, "Cannot pull payload '%ls' as the virtualization system is disabled", *LexToString(Request.GetIdentifier()));
		}
#endif //UE_LOG_ON_NULLSYSTEM_USE

		return false;
	}

	virtual EQueryResult QueryPayloadStatuses(TArrayView<const FIoHash> Ids, EStorageType StorageType, TArray<EPayloadStatus>& OutStatuses) override
	{
		OutStatuses.Reset();

		return EQueryResult::Failure_NotImplemented;
	}

	virtual FVirtualizationResult TryVirtualizePackages(TConstArrayView<FString> PackagePaths, EVirtualizationOptions Options) override
	{
		FVirtualizationResult Result;
		Result.AddError(FText::FromString(TEXT("Calling ::TryVirtualizePackages on FNullVirtualizationSystem")));

		return Result;
	}

	virtual FRehydrationResult TryRehydratePackages(TConstArrayView<FString> PackagePaths, ERehydrationOptions Options) override
	{
		FRehydrationResult Result;
		Result.AddError(FText::FromString(TEXT("Calling ::TryRehydratePackages on FNullVirtualizationSystem")));

		return Result;
	}

	virtual ERehydrationResult TryRehydratePackages(TConstArrayView<FString> PackagePaths, uint64 PaddingAlignment, TArray<FText>& OutErrors, TArray<FSharedBuffer>& OutPackages, TArray<FRehydrationInfo>* OutInfo) override
	{
		OutErrors.Reset();
		OutErrors.Add(FText::FromString(TEXT("Calling ::TryRehydratePackages on FNullVirtualizationSystem")));

		return ERehydrationResult::Failed;
	}

	virtual void DumpStats() const override
	{
		// The null implementation will have no stats and nothing to log
	}

	virtual TArray<FBackendStats> GetBackendStatistics() const override
	{
		return TArray<FBackendStats>();
	}

	virtual FPayloadActivityInfo GetSystemStatistics() const override
	{
		return FPayloadActivityInfo();
	}

	virtual void GatherAnalytics(TArray<FAnalyticsEventAttribute>& Attributes) const override
	{
		// The null implementation has analytics to capture
	}

	virtual FOnNotification& GetNotificationEvent() override
	{
		return NotificationEvent;
	}

	FOnNotification NotificationEvent;
};

TUniquePtr<IVirtualizationSystem> GVirtualizationSystem = nullptr;
FName GVirtualizationSystemName;

/**
 * Utility to check if either cmd is present in the command line. Useful when transitioning from one
 * command line to another.
 */
static bool IsCmdLineSet(const TCHAR* Cmd, const TCHAR* AlternativeCmd = nullptr)
{
	const TCHAR* CmdLine = FCommandLine::Get();

	if (FParse::Param(CmdLine, Cmd))
	{
		return true;
	}

	if (AlternativeCmd != nullptr && FParse::Param(CmdLine, AlternativeCmd))
	{
		return true;
	}

	return false;
}

/** Utility function for finding a IVirtualizationSystemFactory for a given system name */
Private::IVirtualizationSystemFactory* FindFactory(FName SystemName)
{
	TArray<Private::IVirtualizationSystemFactory*> AvaliableSystems = IModularFeatures::Get().GetModularFeatureImplementations<Private::IVirtualizationSystemFactory>(FName("VirtualizationSystem"));
	for (Private::IVirtualizationSystemFactory* SystemFactory : AvaliableSystems)
	{
		if (SystemFactory->GetName() == SystemName)
		{
			return SystemFactory;
		}
	}

	return nullptr;
}

/** Utility determining if the virtualization system should be initialized immediately or on first use */
bool ShouldLazyInitializeSystem(const FConfigFile& ConfigFile)
{
#if UE_VIRTUALIZATION_SYSTEM_LAZY_INIT
	UE_LOGF(LogVirtualization, Display, "The virtualization system will lazy initialize due to code");
	return true;
#else
	if (IsCmdLineSet(TEXT("VALazyInit"), TEXT("VA-LazyInit")))
	{
		UE_LOGF(LogVirtualization, Display, "The virtualization system will lazy initialize due to the command line");
		return true;
	}
	else if (CVarLazyInitSystem.GetValueOnAnyThread())
	{
		UE_LOGF(LogVirtualization, Display, "The virtualization system will lazy initialize due to a cvar");
		return true;
	}
	else
	{
		bool bLazyInit = false;
		ConfigFile.GetBool(TEXT("Core.ContentVirtualization"), TEXT("LazyInit"), bLazyInit);

		if (bLazyInit)
		{
			UE_LOGF(LogVirtualization, Display, "The virtualization system will lazy initialize due to the  ini file option");
			return true;
		}
		else
		{
			return false;
		}
	}
#endif //UE_VIRTUALIZATION_SYSTEM_LAZY_INIT
}

/*** Utility determining which virtualization system should be mounted during initialization */
FName FindSystemToMount(const FConfigFile& ConfigFile)
{
#if UE_DISABLE_VIRTUALIZATION_SYSTEM
	UE_LOGF(LogVirtualization, Display, "The virtualization system has been disabled by code");
	return FName();
#else
	if (IsCmdLineSet(TEXT("VADisable"), TEXT("VA-Disable")))
	{
		UE_LOGF(LogVirtualization, Display, "The virtualization system has been disabled by the command line");
		return FName();
	}
	else if (CVarDisableSystem.GetValueOnAnyThread())
	{
		UE_LOGF(LogVirtualization, Display, "The virtualization system has been disabled by cvar");
		return FName();
	}
	else
	{
		FString SystemName;
		if (ConfigFile.GetString(TEXT("Core.ContentVirtualization"), TEXT("SystemName"), SystemName))
		{
			UE_LOGF(LogVirtualization, Display, "VirtualizationSystem name found in ini file: %ls", *SystemName);
			return FName(SystemName);	
		}
		else
		{
			return FName();
		}
	}
#endif //UE_DISABLE_VIRTUALIZATION_SYSTEM
}

void Initialize(EInitializationFlags Flags)
{
	const FConfigFile* ConfigFile = GConfig->Find(GEngineIni);

	if (ConfigFile != nullptr)
	{
		FInitParams InitParams(FApp::GetProjectName() , *ConfigFile);
		Initialize(InitParams, Flags);
	}
	else
	{
		UE_LOGF(LogVirtualization, Error, "Unable to find a valid engine config file when trying to create the virtualization system");

		FConfigFile EmptyConfigFile;
		FInitParams InitParams(TEXT(""), EmptyConfigFile);

		Initialize(InitParams, Flags);
	}	
}

void Initialize(const FInitParams& InitParams, EInitializationFlags Flags)
{
	// If we are not forcing the initialization check to see if we should lazy
	// initialize or not.
	if (!EnumHasAnyFlags(Flags, EInitializationFlags::ForceInitialize))
	{
		if (ShouldLazyInitializeSystem(InitParams.ConfigFile))
		{
			UE_LOGF(LogVirtualization, Warning, "The 'LazyInit' feature is deprecated as of UE 5.5, consider replacing with 'ini:Engine:[Core.VirtualizationModule]:LazyInitConnections=true'");
			return;
		}
	}

	// Only allow one thread to initialize the system at a time
	static FCriticalSection InitCS;

	FScopeLock _(&InitCS);
	
	if (IVirtualizationSystem::IsInitialized())
	{
		// Another thread initialized the system first
		return;
	}

	FName SystemName = FindSystemToMount(InitParams.ConfigFile);

	if (!SystemName.IsNone())
	{
		Private::IVirtualizationSystemFactory* SystemFactory = FindFactory(SystemName);
		if (SystemFactory != nullptr)
		{
			TUniquePtr<IVirtualizationSystem> NewSystem = SystemFactory->Create();
			check(NewSystem != nullptr); // It is assumed that create will always return a valid pointer

			if (NewSystem->Initialize(InitParams))
			{
				check(!IVirtualizationSystem::IsInitialized());
				GVirtualizationSystem = MoveTemp(NewSystem);
				GVirtualizationSystemName = SystemName;
			}
			else
			{
				UE_LOGF(LogVirtualization, Error, "Initialization of the virtualization system '%ls' failed, falling back to the default implementation", *SystemName.ToString());
			}
		}
		else
		{
			UE_LOGF(LogVirtualization, Error, "Unable to find factory to create the virtualization system: %ls", *SystemName.ToString());
		}
	}

	// If we found no system to create so we will use the fallback system
	if (!GVirtualizationSystem.IsValid())
	{
		TUniquePtr<IVirtualizationSystem> NullSystem = MakeUnique<FNullVirtualizationSystem>();
		NullSystem->Initialize(InitParams);

		check(!IVirtualizationSystem::IsInitialized());
		GVirtualizationSystem = MoveTemp(NullSystem);
	}
}

bool ShouldInitializePreSlate()
{
	bool bInitBeforeSlate = false;
	GConfig->GetBool(TEXT("Core.ContentVirtualization"), TEXT("InitPreSlate"), bInitBeforeSlate, GEngineIni);

	return bInitBeforeSlate;
}

void Shutdown()
{
	GVirtualizationSystem.Reset();
	UE_LOGF(LogVirtualization, Verbose, "UE::Virtualization was shutdown");
}

FAnalyticsRecordEvent& GetAnalyticsRecordEvent()
{
	static FAnalyticsRecordEvent Event;
	return Event;
}

bool IVirtualizationSystem::IsInitialized()
{
	return GVirtualizationSystem != nullptr;
}

FName IVirtualizationSystem::GetSystemName()
{
	return GVirtualizationSystemName;
}

IVirtualizationSystem& IVirtualizationSystem::Get()
{
	// For now allow Initialize to be called directly if it was not called explicitly.
	if (!IsInitialized())
	{
		// We need to ForceInitialize at this point to make sure any lazy init flags are ignored
		UE::Virtualization::Initialize(EInitializationFlags::ForceInitialize);
		check(IsInitialized());
	}

	return *GVirtualizationSystem;
}

} // namespace UE::Virtualization
