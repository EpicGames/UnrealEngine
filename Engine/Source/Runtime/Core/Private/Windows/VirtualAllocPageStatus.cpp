// Copyright Epic Games, Inc. All Rights Reserved.

#include "VirtualAllocPageStatus.h"

#if UE_VIRTUALALLOC_PAGE_STATUS_ENABLED 

#include "Async/UniqueLock.h"
#include "Containers/ArrayView.h"
#include "HAL/PlatformMemory.h"
#include "Math/UnrealMathUtility.h"
#include "Windows/WindowsHWrapper.h"

void FHashMapLinearProbingVAlloc::Init(uint64 PageSize)
{
	// Minimum allocation size is a page, so set initial size to enough buckets to fill a page. Besides this
	// pagesize minimum, we also require a minimum size of 2*CollisionResolutionDeltaFraction for collision resolution.
	Realloc(static_cast<int64>(FMath::Max(2 * CollisionResolutionDeltaFraction, PageSize / sizeof(Buckets[0]))));
}

FHashMapLinearProbingVAlloc::~FHashMapLinearProbingVAlloc()
{
	if (Buckets)
	{
		int64 AllocationSize = NumBuckets * sizeof(Buckets[0]);
		Free(Buckets, AllocationSize);
		LLM_PLATFORM_SCOPE(ELLMTag::FMalloc);
		FLowLevelMemTracker::Get().OnLowLevelChangeInMemoryUse(ELLMTracker::Platform, -AllocationSize);
	}
}

FHashMapLinearProbingVAlloc::EValueType FHashMapLinearProbingVAlloc::FBucketEntry::GetValueType()
{
	if (Key <= MaxKey)
	{
		return EValueType::Active;
	}
	EValueType ValueType = static_cast<EValueType>(Key - (MaxKey + 1));
	LLMCheck(ValueType < EValueType::Active);
	return ValueType;
}

void FHashMapLinearProbingVAlloc::FBucketEntry::SetAsActive(uint64 InKey)
{
	LLMCheck(InKey <= MaxKey);
	Key = InKey;
}

void FHashMapLinearProbingVAlloc::FBucketEntry::SetAsUnallocated()
{
	Key = MaxKey + 1 + static_cast<uint64>(EValueType::Unallocated);
}

void FHashMapLinearProbingVAlloc::FBucketEntry::SetAsTombstone()
{
	Key = MaxKey + 1 + static_cast<uint64>(EValueType::Tombstone);
}

void* FHashMapLinearProbingVAlloc::Malloc(size_t Size)
{
	return VirtualAlloc(nullptr, Size, MEM_COMMIT | MEM_RESERVE, PAGE_READWRITE);
}

void FHashMapLinearProbingVAlloc::Free(void* Addr, size_t Size)
{
	VirtualFree(Addr, 0, MEM_RELEASE);
}

void FHashMapLinearProbingVAlloc::Realloc(int64 InNum)
{
	LLMCheck(FMath::IsPowerOfTwo(InNum));
	LLMCheck(InNum >= 2 * CollisionResolutionDeltaFraction);

	// TODO: Change the tag to ELLMTag::PlatformOverhead, requires LLM to respect PlatformOverhead value instead of clobbering it
	LLM_PLATFORM_SCOPE(ELLMTag::FMalloc);

	int64 OldNumBuckets = NumBuckets;
	FBucketEntry* OldBuckets = Buckets;

	int64 AllocationSize = InNum * sizeof(Buckets[0]);
	NumBuckets = InNum;
	Buckets = reinterpret_cast<FBucketEntry*>(Malloc(AllocationSize));
	FLowLevelMemTracker::Get().OnLowLevelChangeInMemoryUse(ELLMTracker::Platform, AllocationSize);
	for (FBucketEntry& Bucket : TArrayView64<FBucketEntry>(Buckets, NumBuckets))
	{
		Bucket.SetAsUnallocated();
	}
	NumActive = 0;

	if (OldBuckets)
	{
		for (FBucketEntry& OldBucket : TArrayView64<FBucketEntry>(OldBuckets, OldNumBuckets))
		{
			if (OldBucket.GetValueType() == EValueType::Active)
			{
				FindOrAdd(OldBucket.Key, OldBucket.Value);
			}
		}
		int64 OldAllocationSize = OldNumBuckets * sizeof(Buckets[0]);
		Free(OldBuckets, OldAllocationSize);
		FLowLevelMemTracker::Get().OnLowLevelChangeInMemoryUse(ELLMTracker::Platform, -OldAllocationSize);
	}
}

uint64* FHashMapLinearProbingVAlloc::Find(uint64 Key)
{
	FBucketEntry* Bucket = FindBucket(Key);
	if (Bucket)
	{
		return &Bucket->Value;
	}
	return nullptr;
}

FHashMapLinearProbingVAlloc::FBucketEntry* FHashMapLinearProbingVAlloc::FindBucket(uint64 Key)
{
	LLMCheck(Key <= MaxKey);
	int64 FirstBucketIndex = static_cast<int64>(Key & (NumBuckets - 1));
	int64 BucketIndex = FirstBucketIndex;

	// Use the same collision resolution as in FindOrAdd; see the comments in FindOrAdd for explanation of DeltaIndex.
	int64 DeltaIndex = (NumBuckets / CollisionResolutionDeltaFraction) + 1;
	LLMCheck(DeltaIndex % 2 == 1);

	do
	{
		FBucketEntry& Bucket = Buckets[BucketIndex];
		EValueType ValueType = Bucket.GetValueType();
		if (ValueType == EValueType::Active && Bucket.Key == Key)
		{
			return &Bucket;
		}
		else if (ValueType == EValueType::Unallocated)
		{
			return nullptr;
		}
		BucketIndex = (BucketIndex + DeltaIndex) & (NumBuckets - 1);
	} while (BucketIndex != FirstBucketIndex);

	return nullptr;
}

uint64& FHashMapLinearProbingVAlloc::FindOrAdd(uint64 Key, uint64 ValueIfMissing)
{
	LLMCheck(Key <= MaxKey);

	if (NumActive > static_cast<int64>(static_cast<float>(NumBuckets) * DesiredPopulation))
	{
		Realloc(NumBuckets * 2);
		LLMCheck(NumActive < static_cast<int64>(static_cast<float>(NumBuckets) * DesiredPopulation));
	}

	int64 FirstBucketIndex = static_cast<int64>(Key & (NumBuckets - 1));
	int64 BucketIndex = FirstBucketIndex;

	// For collision resolution, we want to jump ahead many pages, because pages are often allocated serially.
	// An important property required of a linear probing rehash algorithm is that it will iterate through all
	// elements of the bucket. For a constant probe distance this is equivalent to the DeltaIndex being relatively
	// prime with NumBuckets. Our NumBuckets is a power of 2, so to be relatively prime we just need to be odd.
	// So we pick a large fraction of NumBuckets that is odd.
	int64 DeltaIndex = (NumBuckets / CollisionResolutionDeltaFraction) + 1;
	LLMCheck(DeltaIndex % 2 == 1);

	do
	{
		FBucketEntry& Bucket = Buckets[BucketIndex];
		EValueType ValueType = Bucket.GetValueType();
		if (ValueType == EValueType::Active && Bucket.Key == Key)
		{
			return Bucket.Value;
		}
		else if (ValueType != EValueType::Active)
		{
			Bucket.SetAsActive(Key);
			Bucket.Value = ValueIfMissing;
			++NumActive;
			LLMCheck(NumActive <= MaxKey);
			return Bucket.Value;
		}
		BucketIndex = (BucketIndex + DeltaIndex) & (NumBuckets - 1);
	} while (BucketIndex != FirstBucketIndex);

	LLMCheckf(false, TEXT("Could not find writable bucket; this is supposed to be prevented by not asking for a writable bucket if the container is full."));
	return Buckets[0].Value;
}

uint64 FHashMapLinearProbingVAlloc::RemoveAndGetValue(uint64 Key, uint64 ValueIfMissing)
{
	FBucketEntry* Bucket = FindBucket(Key);
	if (!Bucket)
	{
		return ValueIfMissing;
	}
	uint64 Result = Bucket->Value;
	LLMCheck(NumActive > 0);
	--NumActive;

	// Use the same collision resolution as in FindOrAdd; see the comments in FindOrAdd for explanation of DeltaIndex.
	int64 DeltaIndex = (NumBuckets / CollisionResolutionDeltaFraction) + 1;
	LLMCheck(DeltaIndex % 2 == 1);

	// If there as an active or tombstone allocation next in the collision resolution series, change this bucket
	// to a tombstone rather than unallocated, so that any elements that tried to add at this bucket but couldn't
	// and therefore moved on to the next position in the collision resolution series can still be found.
	int64 BucketIndex = static_cast<int64>(Bucket - Buckets);
	int64 NextIndex = (BucketIndex + DeltaIndex) & (NumBuckets - 1);
	if (Buckets[NextIndex].GetValueType() != EValueType::Unallocated)
	{
		Bucket->SetAsTombstone();
		return Result;
	}
	Bucket->SetAsUnallocated();

	// Remove any series of tombstones that led up to the bucket we just deallocated
	int64 InverseDeltaIndex = -DeltaIndex;
	int64 PreviousIndex = (BucketIndex + InverseDeltaIndex) & (NumBuckets - 1);
	while (Buckets[PreviousIndex].GetValueType() == EValueType::Tombstone)
	{
		Buckets[PreviousIndex].SetAsUnallocated();
		PreviousIndex = (PreviousIndex + InverseDeltaIndex) & (NumBuckets - 1);
	}

	return Result;
}

FVirtualAllocPageStatus::FVirtualAllocPageStatus()
{
	SYSTEM_INFO SystemInfo;
	FPlatformMemory::Memzero(&SystemInfo, sizeof(SystemInfo));
	::GetSystemInfo(&SystemInfo);

	PageSize = SystemInfo.dwPageSize;
	ReservationAlignment = SystemInfo.dwAllocationGranularity;

	GroupToPageBitsMap.Init(PageSize);
	PageToReservationSizeMap.Init(PageSize);
}

int64 FVirtualAllocPageStatus::MarkChangedAndReturnDeltaSize(void* InStartAddress, SIZE_T InSize, bool bCommitted)
{
	static_assert(sizeof(uint64) >= sizeof(SIZE_T));
	uint64 Size = static_cast<uint64>(InSize);
	static_assert(sizeof(uint64) >= sizeof(void*));
	uint64 StartAddress = reinterpret_cast<uint64>(InStartAddress);

	UE::TUniqueLock Lock(Mutex);

	uint64 StartPage = StartAddress / PageSize;
	uint64 EndPage = (StartAddress + Size + PageSize - 1) / PageSize;

	uint64 StartGroup = StartPage / PagesPerGroup;
	uint64 StartIndexInGroup = StartPage - StartGroup * PagesPerGroup;
	uint64 EndGroup = (EndPage + PagesPerGroup - 1) / PagesPerGroup;
	uint64 EndIndexInGroup = EndPage - (EndGroup - 1) * PagesPerGroup;

	uint64 NumChangedPages = 0;
	uint64 Group = StartGroup;
	uint64 EndFullGroups = EndIndexInGroup == PagesPerGroup ? EndGroup : EndGroup - 1;
	if (StartIndexInGroup != 0)
	{
		// ModifiedBitMask is 1s in bitrange [StartIndexInGroup, PagesPerGroup)
		uint64 ModifiedBitsMask = ~((1ULL << StartIndexInGroup) - 1);
		uint64 NumModifiedBits = PagesPerGroup - StartIndexInGroup;
		if (StartGroup == EndGroup-1 && EndIndexInGroup != PagesPerGroup)
		{
			// ModifiedBitMask is restricted down further to 1s in bitrange [StartIndexInGroup, EndIndexInGroup)
			ModifiedBitsMask = ModifiedBitsMask & ((1ULL << EndIndexInGroup) - 1);
			NumModifiedBits -= PagesPerGroup - EndIndexInGroup;
		}

		if (bCommitted)
		{
			// Count all the unset bits from [StartIndexInGroup,PagesPerGroupOrEndIndex) and set them to 1
			uint64& PageBits = GroupToPageBitsMap.FindOrAdd(Group, 0);
			NumChangedPages += NumModifiedBits - FMath::CountBits64(PageBits & ModifiedBitsMask);
			PageBits |= ModifiedBitsMask;
		}
		else
		{
			// Count all the set bits from [StartIndexInGroup,PagesPerGroupOrEndIndex) and set them to 0
			uint64* PageBits = GroupToPageBitsMap.Find(Group);
			if (PageBits)
			{
				NumChangedPages += FMath::CountBits64((*PageBits) & ModifiedBitsMask);
				*PageBits &= ~ModifiedBitsMask;
				if (*PageBits == 0)
				{
					GroupToPageBitsMap.RemoveAndGetValue(Group, 0);
				}
			}
		}

		++Group;
	}

	while (Group < EndFullGroups)
	{
		if (bCommitted)
		{
			// Count all the unset bits in the bucket and set them to 1
			uint64& PageBits = GroupToPageBitsMap.FindOrAdd(Group, 0);
			NumChangedPages += PagesPerGroup - FMath::CountBits64(PageBits);
			PageBits = ~(0ULL);
		}
		else
		{
			// Count all the set bits in the bucket and set them to 0
			uint64 PageBits = GroupToPageBitsMap.RemoveAndGetValue(Group, 0);
			NumChangedPages += FMath::CountBits64(PageBits);
		}

		++Group;
	}
	if (Group < EndGroup)
	{
		// ModifiedBitMask is 1s in bitrange [0, EndIndexInGroup)
		uint64 ModifiedBitsMask = (1ULL << EndIndexInGroup) - 1;
		uint64 NumModifiedBits = EndIndexInGroup;

		if (bCommitted)
		{
			// Count all the unset bits from [0,EndIndexInGroup) and set them to 1
			uint64& PageBits = GroupToPageBitsMap.FindOrAdd(Group, 0);
			NumChangedPages += NumModifiedBits - FMath::CountBits64(PageBits & ModifiedBitsMask);
			PageBits |= ModifiedBitsMask;
		}
		else
		{
			// Count all the set bits from [0,EndIndexInGroup) and set them to 0
			uint64* PageBits = GroupToPageBitsMap.Find(Group);
			if (PageBits)
			{
				NumChangedPages += FMath::CountBits64((*PageBits) & ModifiedBitsMask);
				*PageBits &= ~ModifiedBitsMask;
				if (*PageBits == 0)
				{
					GroupToPageBitsMap.RemoveAndGetValue(Group, 0);
				}
			}
		}

		++Group;
	}

	int64 DeltaSize = static_cast<int64>(NumChangedPages * PageSize) * (bCommitted ? 1 : -1);
	// Assert no underflow. Underflow is not possible unless there's a bug.
	LLMCheck(DeltaSize >= 0 || AccumulatedSize >= static_cast<uint64>(-DeltaSize));
	// Assert no overflow. Overflow is only possible if we allocate more than 2^64 bytes.
	LLMCheck(DeltaSize <= 0 || MAX_uint64 - AccumulatedSize >= static_cast<uint64>(DeltaSize));
	AccumulatedSize += DeltaSize;
	return DeltaSize;
}


void FVirtualAllocPageStatus::AddReservationSize(void* InReservationAddress, SIZE_T InSize, SIZE_T& OutOldSize)
{
	static_assert(sizeof(uint64) >= sizeof(SIZE_T));
	uint64 Size = static_cast<uint64>(InSize);
	static_assert(sizeof(uint64) >= sizeof(void*));
	uint64 ReservationAddress = reinterpret_cast<uint64>(InReservationAddress);

	UE::TUniqueLock Lock(Mutex);

	uint64 ReservationPage = ReservationAddress / ReservationAlignment;

	uint64& StoredSize = PageToReservationSizeMap.FindOrAdd(ReservationPage, 0);
	OutOldSize = StoredSize;
	StoredSize = InSize;
}

SIZE_T FVirtualAllocPageStatus::GetAndRemoveReservationSize(void* InReservationAddress)
{
	static_assert(sizeof(uint64) >= sizeof(void*));
	uint64 ReservationAddress = reinterpret_cast<uint64>(InReservationAddress);

	UE::TUniqueLock Lock(Mutex);

	uint64 ReservationPage = ReservationAddress / ReservationAlignment;

	return PageToReservationSizeMap.RemoveAndGetValue(ReservationPage, 0);
}

#endif // UE_VIRTUALALLOC_PAGE_STATUS_ENABLED

