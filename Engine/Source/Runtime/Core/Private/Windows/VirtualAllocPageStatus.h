// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "HAL/LowLevelMemTracker.h"

// FVirtualAllocPageTracker is only referenced by LLM-specific code
#define UE_VIRTUALALLOC_PAGE_STATUS_ENABLED ENABLE_LOW_LEVEL_MEM_TRACKER

#if UE_VIRTUALALLOC_PAGE_STATUS_ENABLED

#include "HAL/Platform.h"
#include "HAL/PlatformMutex.h"
#include "Math/NumericLimits.h"

/**
 * A private implementation of a hashmap for FVirtualAllocPageStatus. Maps uint64 -> uint64 and allocates its
 * memory using VirtualAlloc/VirtualFree.
 * 
 * 1) Uses linear probing to resolve collisions - this keeps number of allocations small.
 * 2) Uses VirtualAlloc/VirtualFree to allocate memory - this makes it suitable for being called from inside memory
 * management systems rather than going through FMalloc and encountering reentry during its allocations.
 * 3) Hardcoded with key,value,desiredpopulation rate appropriate for tracking POD payload associated with
 * an address.
 */
struct FHashMapLinearProbingVAlloc
{
public:
	void Init(uint64 PageSize);
	~FHashMapLinearProbingVAlloc();

	uint64* Find(uint64 Key);
	uint64& FindOrAdd(uint64 Key, uint64 ValueIfMissing);

	uint64 RemoveAndGetValue(uint64 Key, uint64 ValueIfMissing);

private:
	enum class EValueType
	{
		Unallocated,
		Tombstone,
		Active, // Any Key value <= MaxKey is an Active address.
	};

	struct FBucketEntry
	{
		uint64 Key;
		uint64 Value;

		EValueType GetValueType();
		void SetAsActive(uint64 InKey);
		void SetAsUnallocated();
		void SetAsTombstone();
	};

private:
	void* Malloc(size_t Size);
	void Free(void* Ptr, size_t Size);
	void Realloc(int64 InNum);
	FBucketEntry* FindBucket(uint64 Key);

private:
	FBucketEntry* Buckets = nullptr;
	int64 NumBuckets = 0;
	int64 NumActive = 0;
	static constexpr float DesiredPopulation = 0.3f;
	static constexpr uint64 MaxKey = MAX_uint64 - static_cast<uint64>(EValueType::Active);
	static constexpr uint64 CollisionResolutionDeltaFraction = 32;
};

/**
 * Tracks the data necessary to record how much memory is committed and decommitted during calls to VirtualAlloc (and
 * its family of similar functions) and Virtual Free. Threadsafe.
 * 
 * (1) Tracks size of the reservation made by VirtualAlloc with MEM_RESERVE, as this size must be looked up during
 * VirtualFree to know how many pages are decommitted and released by MEM_RELEASE.
 * (2) Tracks for every page whether that page has been present in any range passed to VirtualAlloc with MEM_COMMIT
 * without yet being in a range passed to VirtualFree. VirtualAlloc will take no allocation action on pages that are
 * already committed, and VirtualFree will take no allocation action on pages that are already decommitted, so we need
 * to keep track of committed state per page to measure the memory committed/decommitted by those calls.
 */
struct FVirtualAllocPageStatus
{
	FVirtualAllocPageStatus();

	/**
	 * Called from VirtualAlloc with MEM_COMMIT or VirtualFree to record the commit or decommit and report how much
	 * memory was committed or decommitted.
	 */
	int64 MarkChangedAndReturnDeltaSize(void* InStartAddress, SIZE_T InSize, bool bCommitted);
	/** Called from VirtualAlloc with MEM_RESERVE to record the size associated with the returned pointer. */
	void AddReservationSize(void* InReservationAddress, SIZE_T InSize, SIZE_T& OutOldSize);
	/** Called from VirtualFree with MEM_RELEASE to return the size recorded for the freed pointer. */
	SIZE_T GetAndRemoveReservationSize(void* InReservationAddress);

private:
	UE::FPlatformRecursiveMutex Mutex;
	FHashMapLinearProbingVAlloc GroupToPageBitsMap;
	FHashMapLinearProbingVAlloc PageToReservationSizeMap;
	uint64 PageSize = 1;
	uint64 ReservationAlignment = 1;
	uint64 AccumulatedSize = 0;

	static constexpr uint64 PagesPerGroup = sizeof(uint64) * 8;
};

#endif // UE_VIRTUALALLOC_PAGE_STATUS_ENABLED
