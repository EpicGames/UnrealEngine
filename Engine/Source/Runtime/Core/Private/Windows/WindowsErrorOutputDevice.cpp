// Copyright Epic Games, Inc. All Rights Reserved.

#include "Windows/WindowsErrorOutputDevice.h"
#include "Logging/LogMacros.h"
#include "Misc/OutputDevice.h"
#include "HAL/PlatformTime.h"
#include "Math/UnrealMathUtility.h"
#include "HAL/UnrealMemory.h"
#include "Templates/UnrealTemplate.h"
#include "CoreGlobals.h"
#include "Misc/CString.h"
#include "Misc/Parse.h"
#include "Misc/CommandLine.h"
#include "Internationalization/Internationalization.h"
#include "Logging/StructuredLog.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/OutputDeviceHelper.h"
#include "Misc/CoreDelegates.h"
#include "Misc/App.h"
#include "Misc/OutputDeviceRedirector.h"
#include "HAL/ThreadHeartBeat.h"
#include "HAL/ExceptionHandling.h"
#include "Windows/WindowsHWrapper.h"

extern CORE_API bool GIsGPUCrashed;

FWindowsErrorOutputDevice::FWindowsErrorOutputDevice()
{
}

void FWindowsErrorOutputDevice::Serialize( const TCHAR* Msg, ELogVerbosity::Type Verbosity, const class FName& Category )
{
	UE_DEBUG_BREAK();
   
	if( !GIsCriticalError )
	{   
		// First appError.
		GIsCriticalError = 1;

		// There is a bit of ambiguity around which SEH __except is to call HandleError or LogFormattedMessageWithCallstack
		// We print it here so that the message, at very least, is always included with the logs
		UE_LOGFMT(LogWindows, Error, "appError called: {Msg}", Msg);

		// Windows error.
		const int32 LastError = ::GetLastError();
		TCHAR ErrorBuffer[1024];
		ErrorBuffer[0] = TCHAR('\0');
		if (LastError == 0)
		{
			UE_LOGF(LogWindows, Log, "Windows GetLastError: %ls (%i)", FPlatformMisc::GetSystemErrorMessage(ErrorBuffer, 1024, LastError), LastError);
		}
		else
		{
			UE_LOGF(LogWindows, Error, "Windows GetLastError: %ls (%i)", FPlatformMisc::GetSystemErrorMessage(ErrorBuffer, 1024, LastError), LastError);
		}

		// CheckVerifyFailedImpl writes GErrorHist including a callstack and then calls GError->Logf with only the
		// assertion expression and description. Keep GErrorHist intact if it begins with Msg.
		if (FCString::Strncmp(GErrorHist, Msg, FMath::Min<int32>(UE_ARRAY_COUNT(GErrorHist), FCString::Strlen(Msg))))
		{
			FCString::Strncpy(GErrorHist, Msg, UE_ARRAY_COUNT(GErrorHist) - 5);
			FCString::StrncatTruncateDest(GErrorHist, UE_ARRAY_COUNT(GErrorHist), TEXT("\r\n\r\n"));
		}
	}
	else
	{
		UE_LOGF(LogWindows, Error, "Error reentered: %ls", Msg );
	}

	if( GIsGuarded )
	{
		// Propagate error so structured exception handler can perform necessary work.
#if PLATFORM_EXCEPTIONS_DISABLED
		UE_DEBUG_BREAK();
#endif
		void* ErrorProgramCounter = GetErrorProgramCounter();
		if (GIsGPUCrashed)
		{
			ReportGPUCrash(Msg, ErrorProgramCounter);
		}
		else
		{
			ReportAssert(Msg, ErrorProgramCounter);
		}
	}
	else
	{
		// We crashed outside the guarded code (e.g. appExit).
		HandleError();
		FPlatformMisc::RequestExit( true, TEXT("WindowsErrorOutputDevice::Serialize.!GIsGuarded"));
	}
}

void FWindowsErrorOutputDevice::HandleError()
{
	// make sure we don't report errors twice
	static int32 CallCount = 0;
	int32 NewCallCount = FPlatformAtomics::InterlockedIncrement(&CallCount);
	if (NewCallCount != 1)
	{
		UE_LOGF(LogWindows, Error, "HandleError re-entered." );
		return;
	}
	
	GIsGuarded				= 0;
	GIsRunning				= 0;
	GIsCriticalError		= 1;
	GLogConsole				= NULL;
	GErrorHist[UE_ARRAY_COUNT(GErrorHist) - 1] = TCHAR('\0');

	// Trigger the OnSystemFailure hook if it exists
	// make sure it happens after GIsGuarded is set to 0 in case this hook crashes
	FCoreDelegates::OnHandleSystemError.Broadcast();

	// Dump the error and flush the log. If you change behavior here, you should probably update RenderingThread's __except block in FRenderingThread::Run
#if !NO_LOGGING
	FDebug::LogFormattedMessageWithCallstack(LogWindows.GetCategoryName(), __FILE__, __LINE__, TEXT("=== Critical error: ==="), GErrorHist, ELogVerbosity::Error);
#endif
	GLog->Panic();

	HandleErrorRestoreUI();

	FPlatformMisc::SubmitErrorReport( GErrorHist, EErrorReportMode::Interactive );

	FCoreDelegates::OnShutdownAfterError.Broadcast();
}

void FWindowsErrorOutputDevice::HandleErrorRestoreUI()
{
}
