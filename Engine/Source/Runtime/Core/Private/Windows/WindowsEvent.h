// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "HAL/Event.h"
#include "Windows/AllowWindowsPlatformTypes.h"


/**
 * Implements the Windows version of the FEvent interface.
 */
class FEventWin
	: public FEvent
{
public:

	/** Default constructor. */
	FEventWin()
		: Event(nullptr)
	{ }

	/** Virtual destructor. */
	virtual ~FEventWin()
	{
		if (Event != nullptr)
		{
			CloseHandle(Event);
		}
	}

	// FEvent interface

	virtual bool Create( bool bIsManualReset = false ) override
	{
		// Create the event and default it to non-signaled
		Event = CreateEvent(nullptr, bIsManualReset, 0, nullptr);
		ManualReset = bIsManualReset;

		return Event != nullptr;
	}

	virtual bool IsManualReset() override
	{
		return ManualReset;
	}

	virtual void Trigger() override;
	virtual void Reset() override;
	virtual bool Wait( uint32 WaitTime, const bool bIgnoreThreadIdleStats = false ) override;

private:
	/** Holds the handle to the event. */
	HANDLE Event;

	/** Whether the signaled state of the event needs to be reset manually. */
	bool ManualReset;
};


#include "Windows/HideWindowsPlatformTypes.h"
