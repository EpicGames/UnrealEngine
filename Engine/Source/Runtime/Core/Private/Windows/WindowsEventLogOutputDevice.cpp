// Copyright Epic Games, Inc. All Rights Reserved.

#include "Windows/WindowsEventLogOutputDevice.h"
#include "Windows/WindowsHWrapper.h"
#include "Containers/UnrealString.h"
#include "Misc/Parse.h"
#include "Misc/App.h"
#include "CoreGlobals.h"

FWindowsEventLogOutputDevice::FWindowsEventLogOutputDevice()
	: EventLog(NULL)
{
	FString InstanceName;
	FString ServerName;
	// Build a name to uniquely identify this instance
	if (FParse::Value(FCommandLine::Get(),TEXT("-Login="),ServerName))
	{
		InstanceName = FApp::GetProjectName();
		InstanceName += ServerName;
	}
	else
	{
		uint32 ProcID = GetCurrentProcessId();
		InstanceName = FString::Printf(TEXT("%s-PID%d"),FApp::GetProjectName(),ProcID);
	}
	// Open the event log using the name built above
	EventLog = RegisterEventSource(NULL,*InstanceName);
	if (EventLog == NULL)
	{
		UE_LOGF(LogWindows, Error,"Failed to open the Windows Event Log for writing (%d)",GetLastError());
	}
}

FWindowsEventLogOutputDevice::~FWindowsEventLogOutputDevice()
{
	TearDown();
}


void FWindowsEventLogOutputDevice::Serialize(const TCHAR* Buffer, ELogVerbosity::Type Verbosity, const class FName& Category)
{
	if (EventLog != NULL)
	{
		// Only forward errors and warnings to the event log
		switch (Verbosity)
		{
		case ELogVerbosity::Error:
			{
				ReportEventW(EventLog,
					EVENTLOG_ERROR_TYPE,
					NULL,
					0xC0000001L,
					NULL,
					1,
					0,
					&Buffer,
					NULL);
				break;
			}
		case ELogVerbosity::Warning:
			{
				ReportEventW(EventLog,
					EVENTLOG_WARNING_TYPE,
					NULL,
					0x80000002L,
					NULL,
					1,
					0,
					&Buffer,
					NULL);
				break;
			}
		}
	}
}
	
void FWindowsEventLogOutputDevice::Flush(void)
{
}

void FWindowsEventLogOutputDevice::TearDown(void)
{
	if (EventLog != NULL)
	{
		DeregisterEventSource(EventLog);
		EventLog = NULL;
	}
}
