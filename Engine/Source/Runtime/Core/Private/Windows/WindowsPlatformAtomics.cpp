// Copyright Epic Games, Inc. All Rights Reserved.

#include "Windows/WindowsPlatformAtomics.h"
#include "Misc/AssertionMacros.h"
#include "Logging/LogMacros.h"
#include "Templates/UnrealTemplate.h"
#include "CoreGlobals.h"


void FWindowsPlatformAtomics::HandleAtomicsFailure(const TCHAR* InFormat, ...)
{
	TCHAR TempStr[1024];
	GET_TYPED_VARARGS(TCHAR, TempStr, UE_ARRAY_COUNT(TempStr), UE_ARRAY_COUNT(TempStr) - 1, InFormat, InFormat);
	UE_LOGF(LogWindows, Fatal, "%ls", TempStr);
}
