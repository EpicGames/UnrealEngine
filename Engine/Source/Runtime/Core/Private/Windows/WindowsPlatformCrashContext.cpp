// Copyright Epic Games, Inc. All Rights Reserved.

#include "Windows/WindowsPlatformCrashContext.h"

#include "BuildSettings.h"
#include "CoreGlobals.h"
#include "HAL/ExceptionHandling.h"
#include "HAL/FileManager.h"
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformMallocCrash.h"
#include "HAL/PlatformOutputDevices.h"
#include "HAL/PlatformProcess.h"
#include "HAL/PlatformTime.h"
#include "HAL/PlatformTLS.h"
#include "HAL/ThreadHeartBeat.h"
#include "HAL/ThreadManager.h"
#include "Internationalization/Internationalization.h"
#include "Misc/App.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreDelegates.h"
#include "Misc/EngineBuildSettings.h"
#include "Misc/EngineVersion.h"
#include "Misc/FeedbackContext.h"
#include "Misc/FileHelper.h"
#include "Misc/MessageDialog.h"
#include "Misc/OutputDeviceArchiveWrapper.h"
#include "Misc/OutputDeviceFile.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Misc/Paths.h"
#include "Serialization/Archive.h"
#include "Templates/UniquePtr.h"
#include "Templates/UnrealTemplate.h"
#include "Windows/WindowsPlatformMisc.h"
#include "Windows/WindowsPlatformProcess.h"
#include "Windows/WindowsPlatformStackWalk.h"
#include <atomic>
#include <signal.h>

#include "Windows/AllowWindowsPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#include <strsafe.h>
#include <dbghelp.h>
#include <Shlwapi.h>
#include <psapi.h>
#include <tlhelp32.h>
#include <shellapi.h>
THIRD_PARTY_INCLUDES_END
// Windows platform types intentionally not hidden till later

#ifndef UE_LOG_CRASH_CALLSTACK
	#define UE_LOG_CRASH_CALLSTACK 1
#endif

#ifndef USE_CRASH_REPORTER_MONITOR
#define USE_CRASH_REPORTER_MONITOR WITH_EDITOR
#endif

#ifndef NOINITCRASHREPORTER
#define NOINITCRASHREPORTER 0
#endif

#ifndef DISABLE_CRC_SUBMIT_AND_RESTART_BUTTON
#define DISABLE_CRC_SUBMIT_AND_RESTART_BUTTON 0
#endif

#ifndef HANDLE_ABORT_SIGNALS
#define HANDLE_ABORT_SIGNALS 1
#endif

#ifndef WINDOWS_USE_WER
#define WINDOWS_USE_WER 0
#endif 

#define CR_CLIENT_MAX_PATH_LEN 265
#define CR_CLIENT_MAX_ARGS_LEN 256

// MiniDumpWriteDump hangs in ASAN builds and stack walking can race with ASAN's usage of DbgHelp (which is not thread safe).
#if PLATFORM_HAS_ASAN_INCLUDE
#	if defined(__SANITIZE_ADDRESS__)
#		define DISABLE_DEBUG_INFO_DUE_TO_ASAN 1
#	elif defined(__has_feature)
#		if __has_feature(address_sanitizer)
#			define DISABLE_DEBUG_INFO_DUE_TO_ASAN 1
#		endif
#	endif
#endif

#ifndef DISABLE_DEBUG_INFO_DUE_TO_ASAN
#	define DISABLE_DEBUG_INFO_DUE_TO_ASAN 0
#endif

#pragma comment( lib, "version.lib" )
#pragma comment( lib, "Shlwapi.lib" )

LONG WINAPI EngineUnhandledExceptionFilter(LPEXCEPTION_POINTERS ExceptionInfo);
LONG WINAPI UnhandledStaticInitException(LPEXCEPTION_POINTERS ExceptionInfo);

/** Platform specific constants. */
enum EConstants
{
	UE4_MINIDUMP_CRASHCONTEXT = LastReservedStream + 1,
};


/**
 * Code for an assert exception
 */
const uint32 EnsureExceptionCode = ECrashExitCodes::UnhandledEnsure; // Use a rather unique exception code in case SEH doesn't handle it as expected.
const uint32 AssertExceptionCode = 0x4000;
const uint32 GPUCrashExceptionCode = 0x8000;
const uint32 OutOfMemoryExceptionCode = 0xc000;
constexpr double DefaultCrashHandlingTimeoutSecs = 60.0;

namespace {

	/**
	 * Write a Windows minidump to disk
	 * @param The Crash context with its data already serialized into its buffer
	 * @param Path Full path of file to write (normally a .dmp file)
	 * @param ExceptionInfo Pointer to structure containing the exception information
	 * @return Success or failure
	 */

	static TAutoConsoleVariable<bool> CVarMinidumpIncludeReferencedMemory(
		TEXT("WindowsPlatformCrashContext.MinidumpIncludeReferencedMemory"),
		true,
		TEXT("Include referenced memory in minidumps for non-continuable errors (enable the MiniDumpWithIndirectlyReferencedMemory flag).\n"),
		ECVF_Default
	);

	 // #CrashReport: 2014-10-08 Move to FWindowsPlatformCrashContext
	bool WriteMinidump(HANDLE Process, DWORD ThreadId, FWindowsPlatformCrashContext& InContext, const TCHAR* Path, LPEXCEPTION_POINTERS ExceptionInfo)
	{
#if DISABLE_DEBUG_INFO_DUE_TO_ASAN
		return false;
#else
		// Are we calling this in process or from an external process?
		const BOOL bIsClientPointers = Process != GetCurrentProcess();

		// Try to create file for minidump.
		HANDLE FileHandle = CreateFileW(Path, GENERIC_WRITE, 0, NULL, CREATE_ALWAYS, FILE_ATTRIBUTE_NORMAL, NULL);

		if (FileHandle == INVALID_HANDLE_VALUE)
		{
			return false;
		}

		// Initialise structure required by MiniDumpWriteDumps
		MINIDUMP_EXCEPTION_INFORMATION DumpExceptionInfo = {};

		DumpExceptionInfo.ThreadId = ThreadId;
		DumpExceptionInfo.ExceptionPointers = ExceptionInfo;
		DumpExceptionInfo.ClientPointers = bIsClientPointers;

		// CrashContext.runtime-xml is now a part of the minidump file.
		MINIDUMP_USER_STREAM CrashContextStream = { 0 };
		CrashContextStream.Type = UE4_MINIDUMP_CRASHCONTEXT;
		CrashContextStream.BufferSize = (ULONG)(InContext.GetBuffer().Len() * sizeof(InContext.GetBuffer()[0]));
		CrashContextStream.Buffer = (void*)*InContext.GetBuffer();

		MINIDUMP_USER_STREAM_INFORMATION CrashContextStreamInformation = { 0 };
		CrashContextStreamInformation.UserStreamCount = 1;
		CrashContextStreamInformation.UserStreamArray = &CrashContextStream;

		MINIDUMP_TYPE MinidumpType = MiniDumpNormal;//(MINIDUMP_TYPE)(MiniDumpWithPrivateReadWriteMemory|MiniDumpWithDataSegs|MiniDumpWithHandleData|MiniDumpWithFullMemoryInfo|MiniDumpWithThreadInfo|MiniDumpWithUnloadedModules);

		// For ensures by default we use minidump to avoid severe hitches when writing 3GB+ files.
		// However the crash dump mode will remain the same.
		bool bShouldBeFullCrashDump = InContext.IsFullCrashDump();
		if (bShouldBeFullCrashDump)
		{
			MinidumpType = (MINIDUMP_TYPE)(MiniDumpWithFullMemory | MiniDumpWithFullMemoryInfo | MiniDumpWithHandleData | MiniDumpWithThreadInfo );
		}
		else
		{
			const bool bIncludeReferencedMemory = CVarMinidumpIncludeReferencedMemory.GetValueOnAnyThread() && !FGenericCrashContext::IsTypeContinuable(InContext.GetType());
			if (bIncludeReferencedMemory)
			{
				MinidumpType = (MINIDUMP_TYPE)(MiniDumpWithThreadInfo | MiniDumpWithIndirectlyReferencedMemory);
			}
		}

		const BOOL Result = MiniDumpWriteDump(Process, GetProcessId(Process), FileHandle, MinidumpType, ExceptionInfo ? &DumpExceptionInfo : NULL, &CrashContextStreamInformation, NULL);
		CloseHandle(FileHandle);

		return Result == TRUE;
#endif
	}
	
	/** Returns the crash timeout in seconds. */
	FORCEINLINE double GetCrashTimeoutSeconds()
	{
		// 60s is plenty of time to generate a crash report under Windows, but generating minidumps under Wine is much slower
		double TimeoutSeconds = FWindowsPlatformMisc::IsWine() ? 300.0 : DefaultCrashHandlingTimeoutSecs;
		if (GConfig)
		{
			// If available override with configurable value. Negative values are interpreted as infinite wait (not generally recommended)
			GConfig->GetDouble(TEXT("CrashReportClient"), TEXT("CrashHandlingTimeoutSecs"), TimeoutSeconds, GEngineIni);
		}
		return TimeoutSeconds;
	}

}

/**
 * Stores information about an assert that can be unpacked in the exception handler.
 */
struct FAssertInfo
{
	const TCHAR* ErrorMessage;
	void* ProgramCounter;

	FAssertInfo(const TCHAR* InErrorMessage, void* InProgramCounter)
		: ErrorMessage(InErrorMessage)
		, ProgramCounter(InProgramCounter)
	{
	}
};

namespace UE::Core::Private
{
	static TAutoConsoleVariable<bool> CVarForceCrashReportDialogOff(
		TEXT("WindowsPlatformCrashContext.ForceCrashReportDialogOff"),
		false,
		TEXT("If true, force the crash report dialog to not be displayed in the event of a crash."),
		ECVF_Default);

	static TAutoConsoleVariable<bool> CVarScreenshotOnGPUCrash(
		TEXT("WindowsPlatformCrashContext.ScreenshotOnGPUCrash"),
		false,
		TEXT("Take a screenshot of the game window to include in crash reports for GPU crashes (Windows only, cooked client only)\n"),
		ECVF_Default
	);
}

/** Optional attachments */
TArray<FString> OptionalAttachmentFilepaths;

CORE_API void FWindowsPlatformCrashContext::AddOptionalAttachment(const FString& OptionalAttachmentFilepath)
{
	OptionalAttachmentFilepaths.Add(OptionalAttachmentFilepath);
}

void FWindowsPlatformCrashContext::CopyOptionalAttachments(const TCHAR* BaseOutputDirectory) const
{
	// If there are any optional attachments, copy them to the appropriate subdirectory.
	if (OptionalAttachmentFilepaths.IsEmpty())
	{
		return;
	}

	FString OptionalAttachmentsDirectory = FPaths::Combine(BaseOutputDirectory, CrashReportOptionalAttachmentsDirName);
	if (!IFileManager::Get().MakeDirectory(*OptionalAttachmentsDirectory))
	{
		UE_LOGF(LogWindows, Error, "Error creating crash report's optional attachments directory %ls", *OptionalAttachmentsDirectory);
		return;
	}

	bool bHasOptionalAttachments = false;
	for (const FString& OptionalAttachmentFilepath : OptionalAttachmentFilepaths)
	{
		FString OutputFilepath = FPaths::Combine(OptionalAttachmentsDirectory, FPaths::GetCleanFilename(OptionalAttachmentFilepath));
		if (IFileManager::Get().Move(*OutputFilepath, *OptionalAttachmentFilepath))
		{
			bHasOptionalAttachments = true;

		}
		else
		{
			UE_LOGF(LogWindows, Error, "Error moving crash report's optional attachment file %ls to output directory", *OptionalAttachmentFilepath);
		}
	}

	if (bHasOptionalAttachments)
	{
		FString OptionalAttachmentsXML = FPaths::Combine(OptionalAttachmentsDirectory, FPlatformCrashContext::CrashContextRuntimeXMLNameW);
		WriteOptionalAttachmentsXML(OptionalAttachmentsXML);
	}
}

void FWindowsPlatformCrashContext::WriteOptionalAttachmentsXML(const FString& Filepath) const
{
	FString Buffer;

	TCHAR CrashGUID[CrashGUIDLength];
	GetUniqueCrashName(CrashGUID, CrashGUIDLength);

	AddHeader(Buffer);

	// Use the same GUID from the base crash report.
	BeginSection(Buffer, RuntimePropertiesTag);
	AddCrashProperty(Buffer, CrashGuidTag, (const TCHAR*)CrashGUID);
	EndSection(Buffer, RuntimePropertiesTag);

	// Mark this as an update ("patch") request.
	BeginSection(Buffer, RequestDataTag);
	AddCrashProperty(Buffer, TEXT("Type"), TEXT("PATCH"));
	EndSection(Buffer, RequestDataTag);

	AddFooter(Buffer);

	if (!FFileHelper::SaveStringToFile(Buffer, *Filepath, FFileHelper::EEncodingOptions::AutoDetect))
	{
		UE_LOGF(LogWindows, Error, "Error writing optional attachments XML file %ls", *Filepath);
	}
}

/**
* Implement platform specific static cleanup function
*/
void FGenericCrashContext::CleanupPlatformSpecificFiles()
{
	// FPaths functions below requires command line to be initialized
	if (!FCommandLine::IsInitialized())
	{
		return;
	}

	// Manually delete any potential leftover gpu dumps because the crash reporter will upload any leftover crash data from last session
	const FString CrashVideoPath = FPaths::ProjectLogDir() + TEXT("CrashVideo.avi");
	IFileManager::Get().Delete(*CrashVideoPath);
}

void FWindowsPlatformCrashContext::AddPlatformSpecificProperties() const
{
	AddCrashProperty(TEXT("PlatformIsRunningWindows"), 1);
	AddCrashProperty(TEXT("PlatformIsRunningWine"), FWindowsPlatformMisc::IsWine());
	AddCrashProperty(TEXT("PlatformSupportsWindows11"), FWindowsPlatformMisc::SupportsWindows11());
	AddCrashProperty(TEXT("IsRunningOnBattery"), FPlatformMisc::IsRunningOnBattery());
	AddCrashProperty(TEXT("IsUsingBatterySaver"), FWindowsPlatformMisc::IsUsingBatterySaver());
	AddCrashProperty(TEXT("HostArchitecture"), FWindowsPlatformMisc::GetHostArchitecture());
	AddCrashProperty(TEXT("IsDeviceHandheld"), FPlatformMisc::IsDeviceGamingHandheld());

	WIDECHAR DriveName = 0;
	const TCHAR* BaseDir = FWindowsPlatformProcess::BaseDir();
	if (BaseDir && *BaseDir)
	{
		FPlatformString::Convert(&DriveName, 1, BaseDir, 1);
		const FPlatformDriveStats* DriveStats = FWindowsPlatformMisc::GetDriveStats(DriveName);
		if (DriveStats)
		{
			AddCrashProperty(TEXT("DriveStats.Project.Name"), BaseDir);
			AddCrashProperty(TEXT("DriveStats.Project.Type"), LexToString(DriveStats->DriveType));
			AddCrashProperty(TEXT("DriveStats.Project.FreeSpaceKb"), DriveStats->FreeBytes/1024);
		}
	}

	const TCHAR* DownloadDir = FGenericPlatformMisc::GamePersistentDownloadDir();
	if (DownloadDir && *DownloadDir)
	{
		FPlatformString::Convert(&DriveName, 1, DownloadDir, 1);
		const FPlatformDriveStats* DriveStats = FWindowsPlatformMisc::GetDriveStats(DriveName);
		if (DriveStats)
		{
			AddCrashProperty(TEXT("DriveStats.PersistentDownload.Name"), DownloadDir);
			AddCrashProperty(TEXT("DriveStats.PersistentDownload.Type"), LexToString(DriveStats->DriveType));
			AddCrashProperty(TEXT("DriveStats.PersistentDownload.FreeSpaceKb"), DriveStats->FreeBytes / 1024);
		}
	}
}


void FWindowsPlatformCrashContext::CopyPlatformSpecificFiles(const TCHAR* OutputDirectory, void* Context)
{
	FGenericCrashContext::CopyPlatformSpecificFiles(OutputDirectory, Context);

	auto GetRecordCrashDump = [this](const TCHAR* Key)
		{
			bool bRecordCrashDump = true;

			static const TCHAR* SectionName = TEXT("CrashReportClient");

			const TCHAR* CrashConfigFilePath = GetCrashConfigFilePath();
			if (IFileManager::Get().FileExists(CrashConfigFilePath))
			{
				FConfigFile CrashConfigFile;
				CrashConfigFile.Read(CrashConfigFilePath);
				if (CrashConfigFile.GetBool(SectionName, Key, bRecordCrashDump))
				{
					return bRecordCrashDump;
				}
			}

#if UE_EDITOR
			if (GConfig->GetBool(SectionName, Key, bRecordCrashDump, GEditorIni))
			{
				return bRecordCrashDump;
			}
#endif

			GConfig->GetBool(SectionName, Key, bRecordCrashDump, GEngineIni);
			return bRecordCrashDump;
		};

	bool bRecordCrashDump = true;
	switch (Type)
	{
		case ECrashContextType::Stall:
			bRecordCrashDump = GetRecordCrashDump(TEXT("Stall.RecordDump"));
			break;
		case ECrashContextType::Ensure:
			bRecordCrashDump = GetRecordCrashDump(TEXT("Ensure.RecordDump"));
			break;
		case ECrashContextType::AbnormalShutdown:
		case ECrashContextType::Assert:
		case ECrashContextType::Crash:
		case ECrashContextType::GPUCrash:
		case ECrashContextType::Hang:
		case ECrashContextType::OutOfMemory:
		default:
			break;
	}

	if (bRecordCrashDump)
	{
		// Save minidump
		LPEXCEPTION_POINTERS ExceptionInfo = (LPEXCEPTION_POINTERS)Context;
		const FString MinidumpFileName = FPaths::Combine(OutputDirectory, FGenericCrashContext::UEMinidumpName);
		WriteMinidump(ProcessHandle.Get(), CrashedThreadId, *this, *MinidumpFileName, ExceptionInfo);
	}

	// If present, include the crash video
	const FString CrashVideoPath = FPaths::ProjectLogDir() / TEXT("CrashVideo.avi");
	if (IFileManager::Get().FileExists(*CrashVideoPath))
	{
		FString CrashVideoFilename = FPaths::GetCleanFilename(CrashVideoPath);
		const FString CrashVideoDstAbsolute = FPaths::Combine(OutputDirectory, *CrashVideoFilename);
		static_cast<void>(IFileManager::Get().Copy(*CrashVideoDstAbsolute, *CrashVideoPath));	// best effort, so don't care about result: couldn't copy -> tough, no video
	}

	CopyOptionalAttachments(OutputDirectory);
}


namespace
{

static int32 ReportCrashCallCount = 0;
static std::atomic<int32> ReportCallCount(0);

static FORCEINLINE bool CreatePipeWrite(void*& ReadPipe, void*& WritePipe)
{
	SECURITY_ATTRIBUTES Attr = { sizeof(SECURITY_ATTRIBUTES), NULL, true };

	if (!::CreatePipe(&ReadPipe, &WritePipe, &Attr, 0))
	{
		return false;
	}

	if (!::SetHandleInformation(WritePipe, HANDLE_FLAG_INHERIT, 0))
	{
		return false;
	}

	return true;
}

/**
 * Finds the crash reporter binary path. Returns true if the file exists.
 */
bool CreateCrashReportClientPath(TCHAR* OutClientPath, int32 MaxLength)
{
	auto CreateCrashReportClientPathImpl = [&OutClientPath, MaxLength](const TCHAR* CrashReportClientExeName) -> bool
	{
		const TCHAR* EngineDir = FPlatformMisc::EngineDir();
		const TCHAR* BinariesDir = FPlatformProcess::GetBinariesSubdirectory();

		// Find the path to crash reporter binary. Avoid creating FStrings.
		*OutClientPath = TCHAR('\0');
		FCString::StrncatTruncateDest(OutClientPath, MaxLength, EngineDir);
		FCString::StrncatTruncateDest(OutClientPath, MaxLength, TEXT("Binaries/"));
		FCString::StrncatTruncateDest(OutClientPath, MaxLength, BinariesDir);
		FCString::StrncatTruncateDest(OutClientPath, MaxLength, TEXT("/"));
		FCString::StrncatTruncateDest(OutClientPath, MaxLength, CrashReportClientExeName);

		const DWORD Results = GetFileAttributesW(OutClientPath);
		return Results != INVALID_FILE_ATTRIBUTES;
	};

#if WITH_EDITOR
	const TCHAR* CrashReportClientShippingName = TEXT("CrashReportClientEditor.exe");
	const TCHAR* CrashReportClientDevelopmentName = TEXT("CrashReportClientEditor-Win64-Development.exe");
	const TCHAR* CrashReportClientDebugName = TEXT("CrashReportClientEditor-Win64-Debug.exe");
#else
	const TCHAR* CrashReportClientShippingName = TEXT("CrashReportClient.exe");
	const TCHAR* CrashReportClientDevelopmentName = TEXT("CrashReportClient-Win64-Development.exe");
	const TCHAR* CrashReportClientDebugName = TEXT("CrashReportClient-Win64-Debug.exe");
#endif

	if (CreateCrashReportClientPathImpl(CrashReportClientShippingName))
	{
		return true;
	}

#if !(UE_BUILD_TEST || UE_BUILD_SHIPPING)
	if (CreateCrashReportClientPathImpl(CrashReportClientDevelopmentName))
	{
		return true;
	}
	if (CreateCrashReportClientPathImpl(CrashReportClientDebugName))
	{
		return true;
	}
#endif

	return false;
}

/**
 * Launches crash reporter client and creates the pipes for communication.
 */
FProcHandle LaunchCrashReportClient(void** OutWritePipe, void** OutReadPipe, uint32* OutCrashReportClientProcessId)
{
	TCHAR CrashReporterClientPath[CR_CLIENT_MAX_PATH_LEN] = { 0 };
	TCHAR CrashReporterClientArgs[CR_CLIENT_MAX_ARGS_LEN] = { 0 };

	void *PipeChildInRead, *PipeChildInWrite, *PipeChildOutRead, *PipeChildOutWrite;

	if (OutCrashReportClientProcessId)
	{
		*OutCrashReportClientProcessId = 0;
	}

	if (!CreatePipeWrite(PipeChildInRead, PipeChildInWrite) || !FPlatformProcess::CreatePipe(PipeChildOutRead, PipeChildOutWrite))
	{
		return FProcHandle();
	}

	// Pass the endpoints to the creator of the client ...
	*OutWritePipe = PipeChildInWrite;
	*OutReadPipe = PipeChildOutRead;
	
	// ... and the other ends to the child
	{
		TCHAR PipeChildInReadStr[64];
		FCString::Sprintf(PipeChildInReadStr, TFormatSpecifier<uintptr_t>::GetFormatSpecifier(), reinterpret_cast<uintptr_t>(PipeChildInRead));

		TCHAR PipeChildOutWriteStr[64];
		FCString::Sprintf(PipeChildOutWriteStr, TFormatSpecifier<uintptr_t>::GetFormatSpecifier(), reinterpret_cast<uintptr_t>(PipeChildOutWrite));
		
		FCString::Sprintf(CrashReporterClientArgs, TEXT(" -READ=%s -WRITE=%s"), PipeChildInReadStr, PipeChildOutWriteStr);
	}

	{
		TCHAR PidStr[128] = { 0 };
		FCString::Sprintf(PidStr, TEXT(" -MONITOR=%u"), FPlatformProcess::GetCurrentProcessId());
		FCString::StrncatTruncateDest(CrashReporterClientArgs, CR_CLIENT_MAX_ARGS_LEN, PidStr);
	}

	{
		// When CRC is spawned to 'monitor' a process, it creates an analytics session summary that can be merged with the summary of the watched process.
		// The summaries are matched together using this process group ID.
		TCHAR AnalyticsSummaryGuid[128] = { 0 };
		FCString::Sprintf(AnalyticsSummaryGuid, TEXT(" -ProcessGroupId=%s"), *FApp::GetInstanceId().ToString(EGuidFormats::Digits));
		FCString::StrncatTruncateDest(CrashReporterClientArgs, CR_CLIENT_MAX_ARGS_LEN, AnalyticsSummaryGuid);
	}

#if DISABLE_CRC_SUBMIT_AND_RESTART_BUTTON
	// Prevent showing the button if the application cannot be restarted.
	FCString::StrncatTruncateDest(CrashReporterClientArgs, CR_CLIENT_MAX_ARGS_LEN, TEXT(" -HideSubmitAndRestart"));
#endif

	// Parse commandline arguments relevant to pass to the client. Note that since we run this from static initialization
	// FCommandline has not yet been initialized, instead we need to use the OS provided methods.
	{
		LPWSTR* ArgList;
		int ArgCount;
		ArgList = CommandLineToArgvW(GetCommandLineW(), &ArgCount);
		if (ArgList != nullptr)
		{
			for (int It = 0; It < ArgCount; ++It)
			{
				TCHAR Path[MAX_PATH];
				if (FParse::Value(ArgList[It], TEXT("abscrashreportclientlog="), Path, UE_ARRAY_COUNT(Path)))
				{
					FCString::StrncatTruncateDest(CrashReporterClientArgs, CR_CLIENT_MAX_ARGS_LEN, TEXT(" -abslog="));
					FCString::StrncatTruncateDest(CrashReporterClientArgs, CR_CLIENT_MAX_ARGS_LEN, Path);
				}
				
			#if !USE_NULL_RHI
				const bool bHasNullRHIOnCommandline = FParse::Param(ArgList[It], TEXT("nullrhi"));
				if (bHasNullRHIOnCommandline)
			#endif
				{
					FCString::StrncatTruncateDest(CrashReporterClientArgs, CR_CLIENT_MAX_ARGS_LEN, TEXT(" -nullrhi"));
				}

				// Pass through any unattended flag
				if (FParse::Param(ArgList[It], TEXT("unattended")))
				{
					FCString::StrncatTruncateDest(CrashReporterClientArgs, CR_CLIENT_MAX_ARGS_LEN, TEXT(" -unattended"));
				}

				// Whether to clean up crash reports after send
				if (FParse::Param(ArgList[It], TEXT("CleanCrashReports")))
				{
					FCString::StrncatTruncateDest(CrashReporterClientArgs, CR_CLIENT_MAX_ARGS_LEN, TEXT(" -CleanCrashReports"));
				}
			}
		}

		// Tell CRC if we're using low integrity paths (AppData\LocalLow instead of AppData\Local).
		if (FWindowsPlatformProcess::ShouldExpectLowIntegrityLevel())
		{
			FCString::StrncatTruncateDest(CrashReporterClientArgs, CR_CLIENT_MAX_ARGS_LEN, TEXT(" -ExpectLowIntegrityLevel"));
		}
	}

#if WITH_EDITOR // Disaster recovery is only enabled for the Editor. Start the server even if in -game, -server, commandlet, the client-side will not connect (its too soon here to query this executable config).
	{
		// Disaster recovery service command line.
		FString DisasterRecoveryServiceCommandLine;
		DisasterRecoveryServiceCommandLine += FString::Printf(TEXT(" -ConcertServer=\"%s\""), *RecoveryService::GetRecoveryServerName()); // Must be in-sync with disaster recovery client module.

		FCString::StrncatTruncateDest(CrashReporterClientArgs, CR_CLIENT_MAX_ARGS_LEN, *DisasterRecoveryServiceCommandLine);
	}
#endif

	FProcHandle Handle;

	// Launch the crash reporter if the client exists
	if (CreateCrashReportClientPath(CrashReporterClientPath, CR_CLIENT_MAX_PATH_LEN))
	{
		UE::HAL::EProcessId ProcessId;
		UE::HAL::FProcess Process
		{
			{
				.Uri = CrashReporterClientPath,
				.Arguments = CrashReporterClientArgs,
				.bDetached = true,
				.bInheritHandles = true
			},
			ProcessId
		};
		
		if (OutCrashReportClientProcessId)
		{
			*OutCrashReportClientProcessId = std::underlying_type_t<UE::HAL::EProcessId>(ProcessId);
		}
		
		UE_LOGF(LogWindows, Log, "Started CrashReportClient (pid=%d)", std::underlying_type_t<UE::HAL::EProcessId>(ProcessId));

		// The CRC instance launched above will respanwn itself to sever the link with the Editor process group. This way, if the user kills the Editor
		// process group in TaskManager, CRC will not die at the same moment and will be able to capture the Editor exit code and send the session summary.
		if (Process)
		{
			if (FEngineBuildSettings::IsSourceDistribution())
			{
				// Don't wait longer than n seconds for CRC to respawn. This is a workaround for people that did not recompile CrashReportClientEditor after updating/recompiling the Engine/Editor.
				// This is for people that had compiled CRC prior 4.25.0 (13410773) and recompiled the Editor only after updating around 4.25.1.
				::WaitForSingleObject(Process.NativeHandle(), 3000);
			}
			else // Distributed binaries (CRC is expected to be prebuilt).
			{
				// Wait for the intermediate CRC to respawn itself and exit.
				::WaitForSingleObject(Process.NativeHandle(), INFINITE);
			}
			
			// The respanwned CRC process writes its own PID to a file named by this process PID (by parsing the -MONITOR={PID} arguments)
			uint32 RepawnedCrcPid = 0;
			FString PidFilePathname = FString::Printf(TEXT("%sue-crc-pid-%d"), FPlatformProcess::UserTempDir(), FPlatformProcess::GetCurrentProcessId());
			if (TUniquePtr<FArchive> Ar = TUniquePtr<FArchive>(IFileManager::Get().CreateFileReader(*PidFilePathname)))
			{
				*Ar << RepawnedCrcPid;
			}

			// The file is not required anymore.
			IFileManager::Get().Delete(*PidFilePathname, /*RequireExist*/false, /*EvenReadOnly*/true);

			Process = nullptr;

			// Acquire a handle on the final CRC instance responsible to handle the crash/reports, but forbid this process from terminating it in case we try to terminate it by accident (like a stomped handle that would terminate the wrong process)
			// Use the minimum required access rights to avoid creating a SecuritySandbox bypass.
			Handle = RepawnedCrcPid != 0 ? FProcHandle(::OpenProcess(PROCESS_QUERY_LIMITED_INFORMATION | SYNCHRONIZE, 0, RepawnedCrcPid)) : FProcHandle();

			// Update the PID returned to the client.
			if (OutCrashReportClientProcessId != nullptr)
			{
				*OutCrashReportClientProcessId = Handle.IsValid() ? RepawnedCrcPid : 0;
			}
		}
	}

	return Handle;
}

/**
 * Enum indicating whether to run the crash reporter UI
 */
enum class EErrorReportUI
{
	/** Ask the user for a description */
	ShowDialog,

	/** Silently uploaded the report */
	ReportInUnattendedMode	
};

/** This lock is to prevent an ensure and a crash to concurrently report to CrashReportClient (CRC) when CRC is running in background and waiting for crash/ensure (monitor mode). */
static FCriticalSection GMonitorLock;
/**
 * Guard against additional context callbacks crashing
 */
void GuardedDumpAdditionalContext(const TCHAR* CrashDirectoryAbsolute, const FAdditionalCrashContextParams& Params)
{
#if !PLATFORM_SEH_EXCEPTIONS_DISABLED
	__try
#endif
	{
		FGenericCrashContext::DumpAdditionalContext(CrashDirectoryAbsolute, Params);
	}
#if !PLATFORM_SEH_EXCEPTIONS_DISABLED
	__except (TRUE)
	{
		// do nothing in case of an error
		FPlatformMisc::LowLevelOutputDebugString(TEXT("An error occurred while executing addtional crash contexts"));
	}
#endif
}

/**
 * Write required information about the crash to the shared context, and then signal the crash reporter client 
 * running in monitor mode about the crash.
 */
int32 ReportCrashForMonitor(
	LPEXCEPTION_POINTERS ExceptionInfo,
	ECrashContextType Type,
	const TCHAR* ErrorMessage,
	void* ErrorProgramCounter,
	HANDLE CrashingThreadHandle,
	DWORD CrashingThreadId,
	FProcHandle& CrashMonitorHandle,
	FSharedCrashContext* SharedContext,
	void* WritePipe,
	void* ReadPipe,
	EErrorReportUI ReportUI)
{
	// An ensures and a crashes can enter this function concurrently.
	FScopeLock ScopedMonitorLock(&GMonitorLock);

	FGenericCrashContext::CopySharedCrashContext(*SharedContext);

	// Set the platform specific crash context, so that we can stack walk and minidump from
	// the crash reporter client.
	SharedContext->PlatformCrashContext = (void*)ExceptionInfo;

	// Setup up the shared memory area so that the crash report
	SharedContext->CrashType = Type;
	SharedContext->CrashingThreadId = CrashingThreadId;
	SharedContext->ErrorProgramCounter = ErrorProgramCounter;
	SharedContext->ExceptionProgramCounter = ExceptionInfo->ExceptionRecord->ExceptionAddress;

	// Determine UI settings for the crash report. Suppress the user input dialog if we're running in unattended mode
	// or if it is forced of through the WindowsPlatformCrashContext.ForceCrashReportDialogOff console variable.
	// Usage data controls if we want analytics in the crash report client
	// Finally we cannot call some of these functions if we crash during static init, so check if they are initialized.
	bool bNoDialog = ReportUI == EErrorReportUI::ReportInUnattendedMode;
	bNoDialog |= UE::Core::Private::CVarForceCrashReportDialogOff.GetValueOnAnyThread() == true;
	bool bSendUnattendedBugReports = true;
	bool bSendUsageData = true;
	bool bCanSendCrashReport = true;
	// Some projects set this value in non editor builds to automatically send error reports unattended, but display
	// a plain message box in the crash report client. See CRC app code for details.
	bool bImplicitSend = false;
	// IsRunningDedicatedServer will check command line arguments, but only for editor builds. 
#if UE_EDITOR
	if (FCommandLine::IsInitialized())
	{
		bNoDialog |= IsRunningDedicatedServer();
	}
#else
	bNoDialog |= IsRunningDedicatedServer();
#endif

	if (FCommandLine::IsInitialized())
	{
		bNoDialog |= FApp::IsUnattended();
		bNoDialog |= IsRunningDedicatedServer();
	}

	if (GConfig)
	{
		GConfig->GetBool(TEXT("/Script/UnrealEd.CrashReportsPrivacySettings"), TEXT("bSendUnattendedBugReports"), bSendUnattendedBugReports, GEditorSettingsIni);
		GConfig->GetBool(TEXT("/Script/UnrealEd.AnalyticsPrivacySettings"), TEXT("bSendUsageData"), bSendUsageData, GEditorSettingsIni);
		
#if !UE_EDITOR
		if (ReportUI != EErrorReportUI::ReportInUnattendedMode)
		{
			// Only check if we are in a non-editor build
			GConfig->GetBool(TEXT("CrashReportClient"), TEXT("bImplicitSend"), bImplicitSend, GEngineIni);
		}
#endif
	}
	else
	{
		// Crashes before config system is ready (e.g. during static init) we cannot know the user
		// settings for sending unattended. We check for the existense of the marker file from previous
		// sessions, otherwise we cannot send a report at all.
		FString NotAllowedUnattendedBugReportMarkerPath = FPaths::Combine(FWindowsPlatformProcess::ApplicationSettingsDir(), TEXT("NotAllowedUnattendedBugReports"));
		if (::PathFileExistsW(*NotAllowedUnattendedBugReportMarkerPath))
		{
			bSendUnattendedBugReports = false;
		}
	}

#if !UE_EDITOR
	// NOTE: A blueprint-only game packaged from a vanilla engine downloaded from Epic Game Store isn't considered a 'Licensee' version because the engine was built by Epic.
	//       There is no way at the moment do distinguish this case properly.
	if (BuildSettings::IsLicenseeVersion())
	{
		// do not send unattended reports in licensees' builds except for the editor, where it is governed by the above setting
		bSendUnattendedBugReports = false;
		bSendUsageData = false;
	}
#endif

	if (bNoDialog && !bSendUnattendedBugReports)
	{
		// If we shouldn't display a dialog (like for ensures) and the user
		// does not allow unattended bug reports we cannot send the report.
		bCanSendCrashReport = false;
	}

	if (!bCanSendCrashReport)
	{
		return EXCEPTION_CONTINUE_EXECUTION;
	}

	SharedContext->UserSettings.bNoDialog = bNoDialog;
	SharedContext->UserSettings.bSendUnattendedBugReports = bSendUnattendedBugReports;
	SharedContext->UserSettings.bSendUsageData = bSendUsageData;
	SharedContext->UserSettings.bImplicitSend = bImplicitSend;

	SharedContext->SessionContext.bIsExitRequested = IsEngineExitRequested();
	FCString::Strncpy(SharedContext->ErrorMessage, ErrorMessage, CR_MAX_ERROR_MESSAGE_CHARS);

	// Setup all the thread ids and names using snapshot dbghelp. Since it's not possible to 
	// query thread names from an external process.
	uint32 ThreadIdx = 0;
	DWORD CurrentProcessId = GetCurrentProcessId();
	HANDLE ThreadSnapshot = CreateToolhelp32Snapshot(TH32CS_SNAPTHREAD, 0);
	bool bCapturedCrashingThreadId = false;
	if (ThreadSnapshot != INVALID_HANDLE_VALUE)
	{
		// Helper function to capture a thread and store its name and id in the context.
		auto CaptureThread = [&SharedContext, &ThreadIdx](const THREADENTRY32& ThreadEntry)
		{
			SharedContext->ThreadIds[ThreadIdx] = ThreadEntry.th32ThreadID;
			const FString& TmThreadName = FThreadManager::GetThreadName(ThreadEntry.th32ThreadID);
			const TCHAR* ThreadName = TmThreadName.IsEmpty() ? TEXT("Unknown") : *TmThreadName;
			FCString::Strncpy(
				&SharedContext->ThreadNames[ThreadIdx*CR_MAX_THREAD_NAME_CHARS],
				ThreadName,
				CR_MAX_THREAD_NAME_CHARS
			);
			ThreadIdx++;
		};

		THREADENTRY32 ThreadEntry;
		ThreadEntry.dwSize = sizeof(THREADENTRY32);
		if (Thread32First(ThreadSnapshot, &ThreadEntry))
		{
			do
			{
				if (ThreadEntry.th32OwnerProcessID == CurrentProcessId)
				{
					if (ThreadEntry.th32ThreadID == CrashingThreadId)
					{
						// Always capture the crashing thread.
						bCapturedCrashingThreadId = true;
						CaptureThread(ThreadEntry);

						if (FPlatformCrashContext::IsTypeContinuable(Type))
						{
							// Early out for continuable types (ensure/stall etc), only capture the responsible thread for performance reasons. (CRC processing 1 thread vs 256 thread is significant).
							break;
						}
					}
					else if (!FPlatformCrashContext::IsTypeContinuable(Type))
					{
						// Capture the thread if enough entries are left (always keep one entry for the crashing thread).
						if (bCapturedCrashingThreadId || ThreadIdx < CR_MAX_THREADS - 1)
						{
							CaptureThread(ThreadEntry);
						}
					}
				}
			} while (Thread32Next(ThreadSnapshot, &ThreadEntry) && (ThreadIdx < CR_MAX_THREADS));
		}
	}
	SharedContext->NumThreads = ThreadIdx;
	CloseHandle(ThreadSnapshot);

	FString CrashDirectoryAbsolute;
	if (FGenericCrashContext::CreateCrashReportDirectory(SharedContext->SessionContext.CrashGUIDRoot, ReportCallCount++, CrashDirectoryAbsolute))
	{
		FCString::Strncpy(SharedContext->CrashFilesDirectory, *CrashDirectoryAbsolute, CR_MAX_DIRECTORY_CHARS);
		// Copy the log file to output
		FGenericCrashContext::DumpLog(CrashDirectoryAbsolute);

		// Generate some files that needs to be done in the crashing process
		FGenericCrashContext::GenerateFiles(*CrashDirectoryAbsolute);

		// Dump additional context
		FAdditionalCrashContextParams Params =
		{ 
			.CrashedThreadId = SharedContext->CrashingThreadId
		};
		GuardedDumpAdditionalContext(*CrashDirectoryAbsolute, Params);
	}

	// Allow the monitor process to take window focus
	if (const DWORD MonitorProcessId = ::GetProcessId(CrashMonitorHandle.Get()))
	{
		::AllowSetForegroundWindow(MonitorProcessId);
	}

	// Write the shared context to the pipe
	bool bPipeWriteSucceeded = true;
	const uint8* DataIt = (const uint8*)SharedContext;
	const uint8* DataEndIt = DataIt + sizeof(FSharedCrashContext);
	while (DataIt != DataEndIt && bPipeWriteSucceeded)
	{
		int32 OutDataWritten = 0;
		bPipeWriteSucceeded = FPlatformProcess::WritePipe(WritePipe, DataIt, static_cast<int32>(DataEndIt - DataIt), &OutDataWritten);
		DataIt += OutDataWritten;
	}

	if (bPipeWriteSucceeded) // The receiver is not likely to respond if the shared context wasn't successfully written to the pipe.
	{
		double WaitResponseStartTimeSecs = FPlatformTime::Seconds();
		// Wait for a response, saying it's ok to continue
		bool bCanContinueExecution = false;
		int32 ExitCode = 0;
		const double TimeoutSecs = GetCrashTimeoutSeconds();
		// Would like to use TInlineAllocator here to avoid heap allocation on crashes, but it doesn't work since ReadPipeToArray 
		// cannot take array with non-default allocator
		TArray<uint8> ResponseBuffer;
		ResponseBuffer.AddZeroed(16);
		while (!FPlatformProcess::GetProcReturnCode(CrashMonitorHandle, &ExitCode) && !bCanContinueExecution)
		{
			if (FPlatformProcess::ReadPipeToArray(ReadPipe, ResponseBuffer))
			{
				if (ResponseBuffer[0] == 0xd && ResponseBuffer[1] == 0xe &&
					ResponseBuffer[2] == 0xa && ResponseBuffer[3] == 0xd)
				{
					bCanContinueExecution = true;
				}
			}

			// In general, the crash monitor app (CRC) is expected to respond within ~5 seconds, but it might be busy sending an
			// ensure/stall that occurred just before. In some degenerated cases, CRC may hang several minutes and cause the crash
			// reporting thread to timeout and resume the crashing thread, likely resulting in this process to exit or to request it exit.
			const double WaitResponseTimeSecs = FPlatformTime::Seconds() - WaitResponseStartTimeSecs;
			if (IsEngineExitRequested() && (TimeoutSecs >= 0 && WaitResponseTimeSecs >= TimeoutSecs))
			{
				break;
			}
		}
	}

	return EXCEPTION_CONTINUE_EXECUTION;
}

/** 
 * Create a crash report, add the user log and video, and save them a unique the crash folder
 * Launch CrashReportClient.exe to read the report and upload to our CR pipeline
 */
int32 ReportCrashUsingCrashReportClient(FWindowsPlatformCrashContext& InContext, EXCEPTION_POINTERS* ExceptionInfo, EErrorReportUI ReportUI)
{
	// Prevent CrashReportClient from spawning another CrashReportClient.
	const TCHAR* ExecutableName = FPlatformProcess::ExecutableName();
	bool bCanRunCrashReportClient = FCString::Stristr( ExecutableName, TEXT( "CrashReportClient" ) ) == nullptr;

	// Suppress the user input dialog if we're running in unattended mode or if it is forced of through the 
	// WindowsPlatformCrashContext.ForceCrashReportDialogOff console variable.
	bool bNoDialog = FApp::IsUnattended() || ReportUI == EErrorReportUI::ReportInUnattendedMode || IsRunningDedicatedServer();
	bNoDialog |= UE::Core::Private::CVarForceCrashReportDialogOff.GetValueOnAnyThread() == true;

	bool bImplicitSend = false;
#if !UE_EDITOR
	if (GConfig && ReportUI != EErrorReportUI::ReportInUnattendedMode)
	{
		// Only check if we are in a non-editor build
		GConfig->GetBool(TEXT("CrashReportClient"), TEXT("bImplicitSend"), bImplicitSend, GEngineIni);
	}
#endif

	bool bSendUnattendedBugReports = true;
	if (GConfig)
	{
		GConfig->GetBool(TEXT("/Script/UnrealEd.CrashReportsPrivacySettings"), TEXT("bSendUnattendedBugReports"), bSendUnattendedBugReports, GEditorSettingsIni);
	}

	// Controls if we want analytics in the crash report client
	bool bSendUsageData = true;
	if (GConfig)
	{
		GConfig->GetBool(TEXT("/Script/UnrealEd.AnalyticsPrivacySettings"), TEXT("bSendUsageData"), bSendUsageData, GEditorSettingsIni);
	}

#if !UE_EDITOR
	// NOTE: A blueprint-only game packaged from a vanilla engine downloaded from Epic Game Store isn't considered a 'Licensee' version because the engine was built by Epic.
	//       There is no way at the moment do distinguish this case properly.
	if (BuildSettings::IsLicenseeVersion())
	{
		// do not send unattended reports in licensees' builds except for the editor, where it is governed by the above setting
		bSendUnattendedBugReports = false;
		bSendUsageData = false;
	}
#endif

	if (bNoDialog && !bSendUnattendedBugReports)
	{
		bCanRunCrashReportClient = false;
	}

	if( bCanRunCrashReportClient )
	{
		TCHAR CrashReporterClientPath[CR_CLIENT_MAX_PATH_LEN] = { 0 };
		bool bCrashReporterRan = false;

		// Generate Crash GUID
		TCHAR CrashGUID[FGenericCrashContext::CrashGUIDLength];
		InContext.GetUniqueCrashName(CrashGUID, FGenericCrashContext::CrashGUIDLength);
		const FString AppName = InContext.GetCrashGameName();

		FString CrashFolder = FPaths::Combine(*FPaths::ProjectSavedDir(), TEXT("Crashes"), CrashGUID);
		FString CrashFolderAbsolute = IFileManager::Get().ConvertToAbsolutePathForExternalAppForWrite(*CrashFolder);
		if (IFileManager::Get().MakeDirectory(*CrashFolderAbsolute, true))
		{
			// Save crash context
			const FString CrashContextXMLPath = FPaths::Combine(*CrashFolderAbsolute, FPlatformCrashContext::CrashContextRuntimeXMLNameW);
			InContext.SerializeAsXML(*CrashContextXMLPath);

			// Generate some files that needs to be done in the crashing process
			FGenericCrashContext::GenerateFiles(*CrashFolderAbsolute);

			// Generate and copy platform specific files (e.g. minidump) to output directory
			InContext.CopyPlatformSpecificFiles(*CrashFolderAbsolute, (void*) ExceptionInfo);

			// Dump additional context
			FAdditionalCrashContextParams Params =
			{
				.CrashedThreadId = InContext.GetCrashedThreadId()
			};
			GuardedDumpAdditionalContext(*CrashFolderAbsolute, Params);

			// Copy the log file to output
			FGenericCrashContext::DumpLog(CrashFolderAbsolute);

			// Build machines do not upload these automatically since it is not okay to have lingering processes after the build completes.
			if (GIsBuildMachine && !FParse::Param(FCommandLine::Get(), TEXT("AllowCrashReportClientOnBuildMachine")))
			{
				return EXCEPTION_CONTINUE_EXECUTION;
			}

			// Run Crash Report Client
			FString CrashReportClientArguments = FString::Printf(TEXT("\"%s\""), *CrashFolderAbsolute);

			// If the editor setting has been disabled to not send analytics extend this to the CRC
			if (!bSendUsageData)
			{
				CrashReportClientArguments += TEXT(" -NoAnalytics ");
			}

			// Pass nullrhi to CRC when the engine is in this mode to stop the CRC attempting to initialize RHI when the capability isn't available
			bool bNullRHI = !FApp::CanEverRender();

			if (bImplicitSend)
			{
				CrashReportClientArguments += TEXT(" -ImplicitSend");
			}

			if (bNoDialog || bNullRHI)
			{
				CrashReportClientArguments += TEXT(" -Unattended");
			}

			if (bNullRHI)
			{
				CrashReportClientArguments += TEXT(" -nullrhi");
			}

			CrashReportClientArguments += FString(" -AppName=") + AppName;
			CrashReportClientArguments += FString(" -CrashGUID=") + CrashGUID;

			const FString DownstreamStorage = FWindowsPlatformStackWalk::GetDownstreamStorage();
			if (!DownstreamStorage.IsEmpty())
			{
				CrashReportClientArguments += FString(" -DebugSymbols=") + DownstreamStorage;
			}

			// CrashReportClient.exe should run without dragging in binaries from an inherited dll directory
			// So, get the current dll directory for restore and clear before creating process
			TCHAR* CurrentDllDirectory = nullptr;
			DWORD BufferSize = (GetDllDirectory(0, nullptr) + 1) * sizeof(TCHAR);
			
			if (BufferSize > 0)
			{
				CurrentDllDirectory = (TCHAR*) FMemory::Malloc(BufferSize);
				if (CurrentDllDirectory)
				{
					FMemory::Memset(CurrentDllDirectory, 0, BufferSize);
					GetDllDirectory(BufferSize, CurrentDllDirectory);
					SetDllDirectory(nullptr);
				}
			}

			FString AbsCrashReportClientLog;
			if (FParse::Value(FCommandLine::Get(), TEXT("AbsCrashReportClientLog="), AbsCrashReportClientLog))
			{
				CrashReportClientArguments += FString::Format(TEXT(" -abslog=\"{0}\""), { AbsCrashReportClientLog });
			}

			if (CreateCrashReportClientPath(CrashReporterClientPath, CR_CLIENT_MAX_PATH_LEN))
			{
#if !(UE_BUILD_SHIPPING)
				if (FParse::Param(FCommandLine::Get(), TEXT("waitforattachcrc")))
				{
					CrashReportClientArguments += TEXT(" -waitforattach");
				}
#endif

#if DISABLE_CRC_SUBMIT_AND_RESTART_BUTTON
				// Prevent showing the button if the application cannot be restarted.
				CrashReportClientArguments += TEXT(" -HideSubmitAndRestart");
#endif

				bCrashReporterRan = FPlatformProcess::CreateProc(CrashReporterClientPath, *CrashReportClientArguments, true, false, false, NULL, 0, NULL, NULL).IsValid();
			}

			// Restore the dll directory
			if (CurrentDllDirectory)
			{
				SetDllDirectory(CurrentDllDirectory);
				FMemory::Free(CurrentDllDirectory);
			}
		}

		if (!bCrashReporterRan)
		{
			UE_LOGF(LogWindows, Log, "Could not start crash report client using %ls", CrashReporterClientPath);

			if (GWarn != nullptr)
			{
				FPlatformMemory::DumpStats(*GWarn);
			}

			if (!bNoDialog)
			{
				FText MessageTitle(FText::Format(
					NSLOCTEXT("MessageDialog", "AppHasCrashed", "The {0} {1} has crashed and will close"),
					FText::FromString(AppName),
					FText::FromString(FPlatformMisc::GetEngineMode())
					));
				FMessageDialog::Open(EAppMsgType::Ok, FText::FromString(GErrorHist), MessageTitle);
			}
		}
	}

	// Let the system take back over (return value only used by ReportEnsure)
	return EXCEPTION_CONTINUE_EXECUTION;
}

} // end anonymous namespace


#include "Windows/HideWindowsPlatformTypes.h"

// Original code below

#include "Windows/AllowWindowsPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
	#include <ErrorRep.h>
	#include <DbgHelp.h>
THIRD_PARTY_INCLUDES_END
#include "Windows/HideWindowsPlatformTypes.h"

#pragma comment(lib, "Faultrep.lib")

/** 
 * Creates an info string describing the given exception record.
 * See MSDN docs on EXCEPTION_RECORD.
 */
#include "Windows/AllowWindowsPlatformTypes.h"
void CreateExceptionInfoString(EXCEPTION_RECORD* ExceptionRecord, TCHAR* OutErrorString, int32 ErrorStringBufSize)
{
	// #CrashReport: 2014-08-18 Fix FString usage?
	FString ErrorString = "Unhandled Exception: ";

#define HANDLE_CASE(x) case x: ErrorString += TEXT(#x); break;

	switch (ExceptionRecord->ExceptionCode)
	{
	case EXCEPTION_ACCESS_VIOLATION:
		ErrorString += TEXT("EXCEPTION_ACCESS_VIOLATION ");
		if (ExceptionRecord->ExceptionInformation[0] == 0)
		{
			ErrorString += TEXT("reading address ");
		}
		else if (ExceptionRecord->ExceptionInformation[0] == 1)
		{
			ErrorString += TEXT("writing address ");
		}
		ErrorString += FString::Printf(TEXT("0x%016llx"), ExceptionRecord->ExceptionInformation[1]);
		break;
	HANDLE_CASE(EXCEPTION_ARRAY_BOUNDS_EXCEEDED)
	HANDLE_CASE(EXCEPTION_DATATYPE_MISALIGNMENT)
	HANDLE_CASE(EXCEPTION_FLT_DENORMAL_OPERAND)
	HANDLE_CASE(EXCEPTION_FLT_DIVIDE_BY_ZERO)
	HANDLE_CASE(EXCEPTION_FLT_INVALID_OPERATION)
	HANDLE_CASE(EXCEPTION_ILLEGAL_INSTRUCTION)
	HANDLE_CASE(EXCEPTION_INT_DIVIDE_BY_ZERO)
	HANDLE_CASE(EXCEPTION_PRIV_INSTRUCTION)
	HANDLE_CASE(EXCEPTION_STACK_OVERFLOW)
	default:
		ErrorString += FString::Printf(TEXT("0x%08x"), (uint32)ExceptionRecord->ExceptionCode);
	}

	FCString::Strncpy(OutErrorString, *ErrorString, ErrorStringBufSize);

#undef HANDLE_CASE
}

#if HANDLE_ABORT_SIGNALS
static void AbortHandler(int Signal)
{
#if !PLATFORM_SEH_EXCEPTIONS_DISABLED && !NOINITCRASHREPORTER
	__try
	{
		// Force an exception to invoke the crash reporter
		FAssertInfo Info(TEXT("Abort signal received"), PLATFORM_RETURN_ADDRESS());
		ULONG_PTR Arguments[] = { (ULONG_PTR)&Info };
		::RaiseException(AssertExceptionCode, 0, UE_ARRAY_COUNT(Arguments), Arguments);
	}
	__except(ReportCrash(GetExceptionInformation()))
	{
		GIsCriticalError = true;
		FPlatformMisc::RequestExit(true, TEXT("WindowsPlatformCrashContext.AbortHandler"));
	}
#endif
}
#endif



/** 
 * Crash reporting thread. 
 * We process all the crashes on a separate thread in case the original thread's stack is corrupted (stack overflow etc).
 * We're using low level API functions here because at the time we initialize the thread, nothing in the engine exists yet.
 **/
class FCrashReportingThread
{
private:
	enum InputEvent
	{
		IE_Crash = 0,
		IE_StopThread,

		IE_MAX
	};

	/** Thread Id of reporter thread*/
	DWORD ThreadId;
	/** Thread handle to reporter thread */
	HANDLE Thread;
	/** Signals that are consumed inputs - crash, stop request, etc... */
	HANDLE InputEvents[IE_MAX];
	/** Event that signals the crash reporting thread has finished processing the crash */
	HANDLE CrashHandledEvent;

	/** Exception information */
	LPEXCEPTION_POINTERS ExceptionInfo;
	/** ThreadId of the crashed thread */
	DWORD CrashingThreadId;
	/** Handle to crashed thread.*/
	HANDLE CrashingThreadHandle;

	/** Process handle to crash reporter client */
	FProcHandle CrashClientHandle;
	/** Pipe for writing to the monitor process. */
	void* CrashMonitorWritePipe;
	/** Pipe for reading from the monitor process. */
	void* CrashMonitorReadPipe;
	/** The crash report client process ID. */
	uint32 CrashMonitorPid;
	/** Memory allocated for crash context. */
	FSharedCrashContext SharedContext;
	

	/** Thread main proc */
	static DWORD STDCALL CrashReportingThreadProc(LPVOID pThis)
	{
		FCrashReportingThread* This = (FCrashReportingThread*)pThis;
		return This->Run();
	}

	/** Main loop that waits for a crash to trigger the report generation */
	FORCENOINLINE uint32 Run()
	{
#if !PLATFORM_SEH_EXCEPTIONS_DISABLED
		__try
#endif
		{
			FPlatformProcess::SetThreadName(TEXT("CrashReportingThread"));

			bool bStopThreadRequested = false;
			while (!bStopThreadRequested)
			{
				DWORD WaitResult = WaitForMultipleObjects((DWORD)IE_MAX, InputEvents, false, 500);
				if (WaitResult == (WAIT_OBJECT_0 + IE_Crash))
				{
					ResetEvent(CrashHandledEvent);
					HandleCrashInternal();

					ResetEvent(InputEvents[IE_Crash]);
					// Let the thread that crashed know we're done.
					SetEvent(CrashHandledEvent);

					break;
				}
				else if (WaitResult ==  (WAIT_OBJECT_0 + IE_StopThread))
				{
					bStopThreadRequested = true;
				}
				
				if (CrashClientHandle.IsValid() && !FPlatformProcess::IsProcRunning(CrashClientHandle))
				{
					// The crash monitor (CrashReportClient) died unexpectedly. Collect the exit code for analytic purpose.
					int32 CrashMonitorExitCode = 0;
					if (FPlatformProcess::GetProcReturnCode(CrashClientHandle, &CrashMonitorExitCode))
					{
						FGenericCrashContext::SetOutOfProcessCrashReporterExitCode(CrashMonitorExitCode);
						FPlatformProcess::CloseProc(CrashClientHandle);
						CrashClientHandle.Reset();
					}
				}
			}
		}
#if !PLATFORM_SEH_EXCEPTIONS_DISABLED
		__except(EXCEPTION_EXECUTE_HANDLER)
		{
			// The crash reporting thread crashed itself. Exit with a code that the out-of-process monitor will be able to pick up and report into analytics.
			TerminateProcess(GetCurrentProcess(), (UINT)ECrashExitCodes::CrashReporterCrashed);
		}
#endif
		return 0;
	}

	/** Called by the destructor to terminate the thread */
	void Stop()
	{
		SetEvent(InputEvents[IE_StopThread]);
	}

public:
		
	FCrashReportingThread()
		: ThreadId(0)
		, Thread(nullptr)
		, CrashHandledEvent(nullptr)
		, ExceptionInfo(nullptr)
		, CrashingThreadId(0)
		, CrashingThreadHandle(nullptr)
		, CrashMonitorWritePipe(nullptr)
		, CrashMonitorReadPipe(nullptr)
		, CrashMonitorPid(0)
	{
		// Synchronization objects
		for (HANDLE& InputEvent : InputEvents)
		{
			InputEvent = CreateEvent(nullptr, true, 0, nullptr);
		}
		CrashHandledEvent = CreateEvent(nullptr, true, 0, nullptr);

		// Add an exception handler to catch issues during static initialization. It is replaced by the engine handler once
		// the guarded main is entered.
		if (!FPlatformMisc::IsDebuggerPresent())
		{
			::SetUnhandledExceptionFilter(UnhandledStaticInitException);
		}

#if WINDOWS_USE_WER
		// disable the internal crash handling when we're using Windows Error Reporting. This means that all exceptions go
		// to the unhandled exception filter and will be re-thrown out of WinMain to be picked up by Windows Error Reporting
		FPlatformMisc::SetCrashHandlingType(ECrashHandlingType::Disabled);
#endif

#if USE_CRASH_REPORTER_MONITOR
		if (!FPlatformProperties::IsServerOnly())
		{
			CrashClientHandle = LaunchCrashReportClient(&CrashMonitorWritePipe, &CrashMonitorReadPipe, &CrashMonitorPid);
			FMemory::Memzero(SharedContext);
		}
#endif

		// Create a background thread that will process the crash and generate crash reports
		Thread = CreateThread(NULL, 0, CrashReportingThreadProc, this, 0, &ThreadId);
		if (Thread)
		{
			SetThreadPriority(Thread, THREAD_PRIORITY_BELOW_NORMAL);
		}

		if (CrashClientHandle.IsValid())
		{
			FGenericCrashContext::SetOutOfProcessCrashReporterPid(CrashMonitorPid);
		}

		// Register an exception handler for exceptions that aren't handled by any vectored exception handlers or structured exception handlers (__try/__except),
		// especially to capture crash in non-engine-wrapped threads (like native threads) that are usually not guarded with structured exception handling.
		FCoreDelegates::GetPreMainInitDelegate().AddRaw(this, &FCrashReportingThread::RegisterUnhandledExceptionHandler);

#if HANDLE_ABORT_SIGNALS
		// Register a signal handler for abort and terminate occurrences so that abnormal or third party shutdowns can be reported.
		FCoreDelegates::GetPreMainInitDelegate().AddRaw(this, &FCrashReportingThread::RegisterAbortSignalHandler);
#endif
	}

	FORCENOINLINE ~FCrashReportingThread()
	{
		if (Thread)
		{
			// Stop the crash reporting thread
			Stop();
			// 1s should be enough for the thread to exit, otherwise don't bother with cleanup
			if (WaitForSingleObject(Thread, 1000) == WAIT_OBJECT_0)
			{
				CloseHandle(Thread);
			}
			Thread = nullptr;
		}

		FCoreDelegates::GetPreMainInitDelegate().RemoveAll(this);

		for (HANDLE& InputEvent : InputEvents)
		{
			CloseHandle(InputEvent);
			InputEvent = nullptr;
		}

		CloseHandle(CrashHandledEvent);
		CrashHandledEvent = nullptr;

		FPlatformProcess::CloseProc(CrashClientHandle);
		CrashClientHandle.Reset();
	}

	void RegisterUnhandledExceptionHandler()
	{
#if !PLATFORM_SEH_EXCEPTIONS_DISABLED && !NOINITCRASHREPORTER
		::SetUnhandledExceptionFilter(EngineUnhandledExceptionFilter);
#endif
	}

#if HANDLE_ABORT_SIGNALS
	void RegisterAbortSignalHandler()
	{
#if !NOINITCRASHREPORTER
		if (::signal(SIGABRT, AbortHandler) != SIG_ERR)
		{
			UE_LOGF(LogWindows, Log, "Custom abort handler registered for crash reporting.");
		}
		else
		{
			UE_LOGF(LogWindows, Error, "Unable to register custom abort handler for crash reporting.");
		}
#endif
	}
#endif

	DWORD GetReporterThreadId() const
	{
		return ThreadId;
	}

	/** Ensures and Stalls are passed through this. */
	FORCEINLINE int32 OnContinuableEvent(ECrashContextType InType, LPEXCEPTION_POINTERS InExceptionInfo, HANDLE InThreadHandle, uint32 InThreadId, void* ProgramCounter, const TCHAR* ErrorMessage, EErrorReportUI ReportUI)
	{
		if (CrashClientHandle.IsValid() && FPlatformProcess::IsProcRunning(CrashClientHandle))
		{
			return ReportCrashForMonitor(
				InExceptionInfo, 
				InType,
				ErrorMessage, 
				ProgramCounter, 
				InThreadHandle,
				InThreadId,
				CrashClientHandle, 
				&SharedContext, 
				CrashMonitorWritePipe, 
				CrashMonitorReadPipe,
				ReportUI
			);
		}
		else
		{
			FWindowsPlatformCrashContext CrashContext(InType, ErrorMessage);
			CrashContext.SetCrashedProcess(FProcHandle(::GetCurrentProcess()));
			CrashContext.SetCrashedThreadId(InThreadId);
			void* ContextWrapper = FWindowsPlatformStackWalk::MakeThreadContextWrapper(InExceptionInfo->ContextRecord, InThreadHandle);
			CrashContext.CapturePortableCallStack(ProgramCounter, ContextWrapper);
			//CrashContext.CaptureAllThreadContexts(); -> For ensure/stall, don't capture all threads to report and resume quickly.

			return ReportCrashUsingCrashReportClient(CrashContext, InExceptionInfo, ReportUI);
		}
	}

	/** The thread that crashed calls this function which will trigger the CR thread to report the crash */
	FORCEINLINE void OnCrashed(LPEXCEPTION_POINTERS InExceptionInfo)
	{
		ExceptionInfo = InExceptionInfo;
		CrashingThreadId = GetCurrentThreadId();
		CrashingThreadHandle = GetCurrentThread();
		SetEvent(InputEvents[IE_Crash]);
	}

	/** The thread that crashed calls this function to wait for the report to be generated */
	FORCEINLINE bool WaitUntilCrashIsHandled()
	{
		const double TimeoutSeconds = GetCrashTimeoutSeconds();
		const DWORD TimeoutMs = TimeoutSeconds < 0.0f ? INFINITE : (static_cast<DWORD>(TimeoutSeconds * 1000));
		return WaitForSingleObject(CrashHandledEvent, TimeoutMs) == WAIT_OBJECT_0;
	}

	/** Crashes during static init should be reported directly to crash monitor. */
	FORCEINLINE int32 OnCrashDuringStaticInit(LPEXCEPTION_POINTERS InExceptionInfo)
	{
		CrashingThreadId = GetCurrentThreadId();
		CrashingThreadHandle = GetCurrentThread();

		void* ErrorProgramCounter = nullptr;
        if (InExceptionInfo && InExceptionInfo->ExceptionRecord)
        {
            ErrorProgramCounter = InExceptionInfo->ExceptionRecord->ExceptionAddress;
        }

		if (CrashClientHandle.IsValid() && FPlatformProcess::IsProcRunning(CrashClientHandle))
		{
			const ECrashContextType Type = ECrashContextType::Crash;
			const TCHAR* ErrorMessage = TEXT("Crash during static initialization");

			if (!FPlatformCrashContext::IsInitalized())
			{
				FPlatformCrashContext::Initialize();
			}

			ReportCrashForMonitor(
				InExceptionInfo,
				Type,
				ErrorMessage,
				ErrorProgramCounter,
				CrashingThreadHandle,
				CrashingThreadId,
				CrashClientHandle,
				&SharedContext,
				CrashMonitorWritePipe,
				CrashMonitorReadPipe,
				EErrorReportUI::ReportInUnattendedMode
			);
		}

		// Always exit the process after handling crash during static initialization.
		ExitProcess(ECrashExitCodes::CrashDuringStaticInit);
	}

private:

	/** Handles the crash */
	FORCENOINLINE void HandleCrashInternal()
	{
		// Stop the heartbeat thread so that it doesn't interfere with crashreporting
		if (FThreadHeartBeat* HB = FThreadHeartBeat::GetNoInit())
		{
			HB->Stop();
		}

		// Then try run time crash processing and broadcast information about a crash.
		FCoreDelegates::OnHandleSystemError.Broadcast();

		// Get the default settings for the crash context
		ECrashContextType Type = ECrashContextType::Crash;
		const TCHAR* ErrorMessage = TEXT("Unhandled exception");
		TCHAR ErrorMessageLocal[UE_ARRAY_COUNT(GErrorExceptionDescription)];
		void* ErrorProgramCounter = ExceptionInfo->ExceptionRecord->ExceptionAddress;

		void* ContextWrapper = nullptr;

		// If it was an assert or GPU crash, allow overriding the info from the exception parameters
		if (ExceptionInfo->ExceptionRecord->ExceptionCode == AssertExceptionCode && ExceptionInfo->ExceptionRecord->NumberParameters == 1)
		{
			const FAssertInfo& Info = *(const FAssertInfo*)ExceptionInfo->ExceptionRecord->ExceptionInformation[0];
			Type = ECrashContextType::Assert;
			ErrorMessage = Info.ErrorMessage;
			ErrorProgramCounter = Info.ProgramCounter;
		}
		else if (ExceptionInfo->ExceptionRecord->ExceptionCode == GPUCrashExceptionCode && ExceptionInfo->ExceptionRecord->NumberParameters == 1)
		{
			const FAssertInfo& Info = *(const FAssertInfo*)ExceptionInfo->ExceptionRecord->ExceptionInformation[0];
			Type = ECrashContextType::GPUCrash;
			ErrorMessage = Info.ErrorMessage;
			ErrorProgramCounter = Info.ProgramCounter;
		}
		else if (ExceptionInfo->ExceptionRecord->ExceptionCode == EnsureExceptionCode)
		{
			const FAssertInfo& Info = *(const FAssertInfo*)ExceptionInfo->ExceptionRecord->ExceptionInformation[0];
			Type = ECrashContextType::Ensure;
			ErrorMessage = Info.ErrorMessage;
			ErrorProgramCounter = Info.ProgramCounter;
		}
		else if (ExceptionInfo->ExceptionRecord->ExceptionCode == OutOfMemoryExceptionCode)
		{
			const FAssertInfo& Info = *(const FAssertInfo*)ExceptionInfo->ExceptionRecord->ExceptionInformation[0];
			Type = ECrashContextType::OutOfMemory;
			ErrorMessage = Info.ErrorMessage;
			ErrorProgramCounter = Info.ProgramCounter;
		}
		// Generic exception description is stored in GErrorExceptionDescription
		else
		{
			// When a generic exception is thrown, it is important to get all the stack frames
			CreateExceptionInfoString(ExceptionInfo->ExceptionRecord, ErrorMessageLocal, UE_ARRAY_COUNT(ErrorMessageLocal));
			ErrorMessage = ErrorMessageLocal;

			// TODO: Fix race conditions when writing GErrorExceptionDescription (concurrent threads can read/write it)
			FCString::Strncpy(GErrorExceptionDescription, ErrorMessageLocal, UE_ARRAY_COUNT(GErrorExceptionDescription));
		}

// Workaround for non-Editor build. When remote debugging was enabled (CRC generating the minidump + callstack) in games, several crashes
// were emitted with no call stack and zero-sized minidump. Until this issue is resolved, games bypass the remote debugging and fallback to
// in-process callstack/minidump generation, then spawn a new instance of CRC to send that crash. The CRC spanwed at startup will keep running in
// background with the only purpose to capture the monitored process exit code and send the analytic summary event once the process died.
#if USE_CRASH_REPORTER_MONITOR && WITH_EDITOR
		if (CrashClientHandle.IsValid() && FPlatformProcess::IsProcRunning(CrashClientHandle))
		{
			// If possible use the crash monitor helper class to report the error. This will do most of the analysis
			// in the crash reporter client process.
			ReportCrashForMonitor(
				ExceptionInfo,
				Type,
				ErrorMessage,
				ErrorProgramCounter,
				CrashingThreadHandle,
				CrashingThreadId,
				CrashClientHandle,
				&SharedContext,
				CrashMonitorWritePipe,
				CrashMonitorReadPipe,
				EErrorReportUI::ShowDialog
			);
		}
		else
#endif
		{
			// Not super safe due to dynamic memory allocations, but at least enables new functionality.
			// Introduces a new runtime crash context. Will replace all Windows related crash reporting.
			FWindowsPlatformCrashContext CrashContext(Type, ErrorMessage);

			// Thread context wrapper for stack operations
			ContextWrapper = FWindowsPlatformStackWalk::MakeThreadContextWrapper(ExceptionInfo->ContextRecord, CrashingThreadHandle);
			CrashContext.SetCrashedProcess(FProcHandle(::GetCurrentProcess()));
			CrashContext.SetCrashedThreadId(CrashingThreadId);
			CrashContext.CaptureAllThreadContexts();
			CrashContext.CapturePortableCallStack(ErrorProgramCounter, ContextWrapper);

			// First launch the crash reporter client.
			if (GUseCrashReportClient)
			{
				ReportCrashUsingCrashReportClient(CrashContext, ExceptionInfo, EErrorReportUI::ShowDialog);
			}
			else
			{
				CrashContext.SerializeContentToBuffer();
				WriteMinidump(GetCurrentProcess(), GetCurrentThreadId(), CrashContext, MiniDumpFilenameW, ExceptionInfo);
			}
		}

		const bool bGenerateRuntimeCallstack =
#if UE_LOG_CRASH_CALLSTACK
			true;
#else
			FParse::Param(FCommandLine::Get(), TEXT("ForceLogCallstacks")) || FEngineBuildSettings::IsInternalBuild() || FEngineBuildSettings::IsPerforceBuild() || FEngineBuildSettings::IsSourceDistribution();
#endif // UE_LOG_CRASH_CALLSTACK
		if (bGenerateRuntimeCallstack)
		{
			const SIZE_T StackTraceSize = 65535;
			ANSICHAR* StackTrace = (ANSICHAR*)UE::Private::GMalloc->Malloc(StackTraceSize);
			StackTrace[0] = 0;
			// Walk the stack and dump it to the allocated memory. This process usually allocates a lot of memory.
			if (!ContextWrapper)
			{
				ContextWrapper = FWindowsPlatformStackWalk::MakeThreadContextWrapper(ExceptionInfo->ContextRecord, CrashingThreadHandle);
			}
			
#if DISABLE_DEBUG_INFO_DUE_TO_ASAN
			FCStringAnsi::Snprintf(StackTrace, StackTraceSize, "(callstack disabled due to ASAN)");
#else
			FPlatformStackWalk::StackWalkAndDump(StackTrace, StackTraceSize, ErrorProgramCounter, ContextWrapper);
#endif
			
			if (ExceptionInfo->ExceptionRecord->ExceptionCode != EnsureExceptionCode && ExceptionInfo->ExceptionRecord->ExceptionCode != AssertExceptionCode)
			{
				CreateExceptionInfoString(ExceptionInfo->ExceptionRecord, GErrorExceptionDescription, UE_ARRAY_COUNT(GErrorExceptionDescription));
				FCString::StrncatTruncateDest(GErrorHist, UE_ARRAY_COUNT(GErrorHist), GErrorExceptionDescription);
				FCString::StrncatTruncateDest(GErrorHist, UE_ARRAY_COUNT(GErrorHist), TEXT("\r\n\r\n"));
			}

			FCString::StrncatTruncateDest(GErrorHist, UE_ARRAY_COUNT(GErrorHist), ANSI_TO_TCHAR(StackTrace));

			UE::Private::GMalloc->Free(StackTrace);
		}

		// Make sure any thread context wrapper is released
		if (ContextWrapper)
		{
			FWindowsPlatformStackWalk::ReleaseThreadContextWrapper(ContextWrapper);
		}

#if !UE_BUILD_SHIPPING
		FPlatformStackWalk::UploadLocalSymbols();
#endif
	}
};

#include "Windows/HideWindowsPlatformTypes.h"

#if !NOINITCRASHREPORTER
TOptional<FCrashReportingThread> GCrashReportingThread(InPlace);
#endif

LONG WINAPI UnhandledStaticInitException(LPEXCEPTION_POINTERS ExceptionInfo)
{
#if !NOINITCRASHREPORTER
	// If we get an exception during static init we hope that the crash reporting thread
	// object has been created, otherwise we cannot handle the exception. This will hopefully 
	// work even if there is a stack overflow. See 
	// https://peteronprogramming.wordpress.com/2016/08/10/crashes-you-cant-handle-easily-2-stack-overflows-on-windows/
	// @note: Even if the object has been created, the actual thread has not been started yet,
	// (that happens after static init) so we must bypass that and report directly from this thread. 
	// 
	if (GCrashReportingThread.IsSet())
	{
		return GCrashReportingThread->OnCrashDuringStaticInit(ExceptionInfo);
	}
#endif

	return EXCEPTION_CONTINUE_SEARCH;
}

/**
 * Fallback for handling exceptions that aren't handled elsewhere.
 *
 * The SEH mechanism is not very well documented, so to start with, few facts to know:
 *   - SEH uses 'handlers' and 'filters'. They have different roles and are invoked at different state.
 *   - Any unhandled exception is going to terminate the program whether it is a benign exception or a fatal one.
 *   - Vectored exception handlers, Vectored continue handlers and the unhandled exception filter are global to the process.
 *   - Exceptions occurring in a thread doesn't automatically halt other threads. Exception handling executes in thread where the exception fired. The other threads continue to run.
 *   - Several threads can crash concurrently.
 *   - Not all exceptions are equal. Some exceptions can be handled doing nothing more than catching them and telling the code to continue (like some user defined exception), some
 *     needs to be handled in a __except() clause to allow the program to continue (like access violation) and others are fatal and can only be reported but not continued (like stack overflow).
 *   - Not all machines are equal. Different exceptions may be fired on different machines for the same usage of the program. This seems especially true when
 *     using the OS 'open file' dialog where the user specific extensions to the Windows Explorer get loaded in the process.
 *   - If an exception handler/filter triggers another exception, the new inner exception is handled recursively. If the code is not robust, it may retrigger that inner exception over and over.
 *     This eventually stops with a stack overflow, at which point the OS terminates the program and the original exception is lost.
 *
 * Usually, when an exception occurs, Windows executes following steps (see below for unusual cases):
 *     1- Invoke the vectored exception handlers registered with AddVectoredExceptionHandler(), if any.
 *         - In general, this is too soon to handle an exception because local structured exception handlers did not execute yet and many exceptions are handled there.
 *         - If a registered vectored exception handler returns EXCEPTION_CONTINUE_EXECUTION, the vectored continue handler(s), are invoked next (see number 4 below)
 *         - If a registered vectored exception handler returns EXCEPTION_CONTINUE_SEARCH, the OS skip this one and continue iterating the list of vectored exception handlers.
 *         - If a registered vectored exception handler returns EXCEPTION_EXECUTE_HANDLER, in my tests, this was equivalent to returning EXCEPTION_CONTINUE_SEARCH.
 *         - If no vectored exception handlers are registered or all registered one return EXCEPTION_CONTINUE_SEARCH, the structured exception handlers (__try/__except) are executed next.
 *         - At this stage, be careful when returning EXCEPTION_CONTINUE_EXECUTION. For example, continuing after an access violation would retrigger the exception immediatedly.
 *     2- If the exception wasn't handled by a vectored exception handler, invoke the structured exception handlers (the __try/__except clauses)
 *         - That let the code manage exceptions more locally, for the Engine, we want that to run first.
 *         - When the filter expression in __except(filterExpression) { block } clause returns EXCEPTION_EXECUTE_HANDLER, the 'block' is executed, the code continue after the block. The exception is considered handled.
 *         - When the filter expression in __except(filterExpression) { block } clause returns EXCEPTION_CONTINUE_EXECUTION, the 'block' is not executed and vectored continue exceptions handlers (if any) gets called. (see number 4 below)
 *         - When the filter expression in __except(filterExpression) { block } clause returns EXCEPTION_CONTINUE_SEARCH, the 'block' is not executed and the search continue for the next __try/__except in the callstack.
 *         - If all unhandled exception filters within the call stack were executed and all of them returned returned EXCEPTION_CONTINUE_SEARCH, the unhandled exception filter is invoked. (see number 3 below)
 *         - The __except { block } allows the code to continue from most exceptions, even from an access violation because code resume after the except block, not at the point of the exception.
 *     3- If the exception wasn't handled yet, the system calls the function registered with SetUnhandedExceptionFilter(). There is only one such function, the last to register override the previous one.
 *         - At that point, both vectored exception handlers and structured exception handlers have had a chance to handle the exception but did not.
 *         - If this function returns EXCEPTION_CONTINUE_SEARCH or EXCEPTION_EXECUTE_HANDLER, by default, the OS handler is invoked and the program is terminated.
 *         - If this function returns EXCEPTION_CONTINUE_EXECUTION, the vectored continue handlers are invoked (see number 4 below)
 *     4- If a handler or a filter returned the EXCEPTION_CONTINUE_EXECUTION, the registered vectored continue handlers are invoked.
 *         - This is last chance to do something about an exception. The program was allowed to continue by a previous filter/handler, effectively ignoring the exception.
 *         - The handler can return EXCEPTION_CONTINUE_SEARCH to observe only. The OS will continue and invoke the next handler in the list.
 *         - The handler can short cut other continue handlers by returning EXCEPTION_CONTINUE_EXECUTION which resume the code immediatedly.
 *         - In my tests, if a vectored continue handler returns EXCEPTION_EXECUTE_HANDLER, this is equivalent to returning EXCEPTION_CONTINUE_SEARCH.
 *         - By default, if no handlers are registered or all registered handler(s) returned EXCEPTION_CONTINUE_SEARCH, the program resumes execution at the point of the exception.
 *
 * Inside a Windows OS callback, in a 64-bit application, a different flow than the one described is used.
 *    - 64-bit applications don't cross Kernel/user-mode easily. If the engine crash during a Kernel callback, EngineUnhandledExceptionFilter() is called directly. This behavior is
 *      documented by various article on the net. See: https://stackoverflow.com/questions/11376795/why-cant-64-bit-windows-unwind-user-kernel-user-exceptions.
 *    - On early versions of Windows 7, the kernel could swallow exceptions occurring in kernel callback just as if they never occurred. This is not the case anymore with Win 10.
 *
 * Other SEH particularities:
 *     - A stack buffer overflow bypasses SEH entirely and the application exits with code: -1073740791 (STATUS_STACK_BUFFER_OVERRUN).
 *     - A stack overflow exception occurs when not enough space remains to push what needs to be pushed, but it doesn't means it has no stack space left at all. The exception will be reported
 *       if enough stack space is available to call/run SEH, otherwise, the app exits with code: -1073741571 (STATUS_STACK_OVERFLOW)
 *     - Fast fail exceptions bypasse SEH entirely and the application exits with code: -1073740286 (STATUS_FAIL_FAST_EXCEPTION) or 1653 (ERROR_FAIL_FAST_EXCEPTION)
 *     - Heap corruption (like a double free) is a special exception. It is likely only visible to Vectored Exception Handler (VEH) before possibly beeing handled by Windows Error Reporting (WER).
 *       A popup may be shown asking to debug or exit. The application may exit with code -1073740940 (STATUS_HEAP_CORRUPTION) or 255 (Abort) depending on the situation.
 *
 * The engine hooks itself in the unhandled exception filter. This is the best place to be as it runs after structured exception handlers and
 * it can be easily overriden externally (because there can only be one) to do something else.
 */
LONG WINAPI EngineUnhandledExceptionFilter(LPEXCEPTION_POINTERS ExceptionInfo)
{
	ReportCrash(ExceptionInfo);

#if WINDOWS_USE_WER
	return EXCEPTION_CONTINUE_SEARCH;
#else

	GIsCriticalError = true;
	if (!IsEngineExitRequested())
	{
		FPlatformMisc::RequestExit(true, TEXT("WindowsPlatformCrashContext.EngineUnhandledExceptionFilter"));
	}
	return EXCEPTION_CONTINUE_SEARCH; // Not really important, RequestExit() terminates the process just above.
#endif // WINDOWS_USE_WER
}

// #CrashReport: 2015-05-28 This should be named EngineCrashHandler
int32 ReportCrash( LPEXCEPTION_POINTERS ExceptionInfo )
{
#if !NOINITCRASHREPORTER
	// Only create a minidump the first time this function is called.
	// (Can be called the first time from the RenderThread, then a second time from the MainThread.)
	if (GCrashReportingThread.IsSet())
	{
		if (GLog)
		{
			// Panic before reporting the crash to make sure that the logs are flushed as thoroughly as possible.
			GLog->Panic();
		}

		if (FPlatformAtomics::InterlockedIncrement(&ReportCrashCallCount) == 1)
		{
			GCrashReportingThread->OnCrashed(ExceptionInfo);
		}

		// Wait 60s for the crash reporting thread to process the message
		GCrashReportingThread->WaitUntilCrashIsHandled();
	}
#endif

#if WINDOWS_USE_WER
	return EXCEPTION_CONTINUE_SEARCH;
#else
	return EXCEPTION_EXECUTE_HANDLER;
#endif // WINDOWS_USE_WER
}

static FCriticalSection EnsureLock;
static bool bReentranceGuard = false;

/**
 * A wrapper for ReportCrashUsingCrashReportClient that creates a new ensure crash context
 */
int32 ReportContinuableEventUsingCrashReportClient(ECrashContextType InType, EXCEPTION_POINTERS* ExceptionInfo, HANDLE InThreadHandle, uint32 InThreadId, void* ProgramCounter, const TCHAR* ErrorMessage, EErrorReportUI ReportUI)
{
#if !NOINITCRASHREPORTER
	return GCrashReportingThread->OnContinuableEvent(InType, ExceptionInfo, InThreadHandle, InThreadId, ProgramCounter, ErrorMessage, ReportUI);
#else 
	return EXCEPTION_EXECUTE_HANDLER;
#endif
}

static void ReportEventOnCallingThread(ECrashContextType InType, const TCHAR* ErrorMessage, void* ProgramCounter)
{
#if !PLATFORM_SEH_EXCEPTIONS_DISABLED
	__try
#endif
	{
		FAssertInfo Info(ErrorMessage, ProgramCounter);
		ULONG_PTR Arguments[] = { (ULONG_PTR)&Info };
		::RaiseException(EnsureExceptionCode, 0, UE_ARRAY_COUNT(Arguments), Arguments);
	}
#if !PLATFORM_SEH_EXCEPTIONS_DISABLED
	__except (ReportContinuableEventUsingCrashReportClient( InType, GetExceptionInformation(), GetCurrentThread(), GetCurrentThreadId(), ProgramCounter, ErrorMessage, IsInteractiveEnsureMode() ? EErrorReportUI::ShowDialog : EErrorReportUI::ReportInUnattendedMode))
		CA_SUPPRESS(6322)
	{
	}
#endif
}

static void ReportEvent(ECrashContextType InType, const TCHAR* ErrorMessage, uint32 InThreadId, void* ProgramCounter)
{
	if (ReportCrashCallCount > 0 || FDebug::HasAsserted())
	{
		// Don't report ensures after we've crashed/asserted, they simply may be a result of the crash as
		// the engine is already in a bad state.
		return;
	}

	// Serialize concurrent ensures (from concurrent threads).
	FScopeLock ScopeLock(&EnsureLock);

	// Ignore any ensure that could be fired by the code reporting an ensure.
	TGuardValue<bool> ReentranceGuard(bReentranceGuard, true);
	if (ReentranceGuard.GetOriginalValue()) // Read the old value.
	{
		return; // Already handling an ensure.
	}

	// Stop checking heartbeat for this thread (and stop the gamethread hitch detector if we're the game thread).
	// Ensure can take a lot of time (when stackwalking), so we don't want hitches/hangs firing.
	// These are no-ops on threads that didn't already have a heartbeat etc.
	FSlowHeartBeatScope SuspendHeartBeat(true);
	FDisableHitchDetectorScope SuspendGameThreadHitch;

	/** This is the last place to gather memory stats before exception. */
	FGenericCrashContext::SetMemoryStats(FPlatformMemory::GetStats());

	if (FPlatformTLS::GetCurrentThreadId() == InThreadId)
	{
		ReportEventOnCallingThread(InType, ErrorMessage, ProgramCounter);
	}
	else
	{
		// This is what is normally returned by GetExceptionInformation(), which is a special compiler intrinsic only valid inside __except parenthesis
		//  Here we construct one for a perfectly valid suspended thread of execution, and pass it into the CrashReportClient pathway
		EXCEPTION_POINTERS ExceptionInformation = {};

		HANDLE ThreadHandle = OpenThread(THREAD_SUSPEND_RESUME|THREAD_GET_CONTEXT|THREAD_QUERY_INFORMATION, 0, InThreadId);
		if (ThreadHandle != nullptr)
		{
			// This is the thread state data, represents the register file state once suspended
			CONTEXT ThreadContext;
			ZeroMemory(&ThreadContext, sizeof(ThreadContext));

			// This include segments and debug registers, just in case
			ThreadContext.ContextFlags |= CONTEXT_ALL;

			DWORD SuspendCount = SuspendThread(ThreadHandle);
			if (SuspendCount != -1)
			{
				// Read the context block from the thread now its suspended
				BOOL bGotThreadContext = GetThreadContext(ThreadHandle, &ThreadContext);
				if (bGotThreadContext)
				{
					// Success! Set the pointer to this state in the exception information block
					ExceptionInformation.ContextRecord = &ThreadContext;

					if (ProgramCounter == nullptr)
					{
#if PLATFORM_CPU_X86_FAMILY || PLATFORM_WINDOWS_ARM64EC
						ProgramCounter = (void*)(ThreadContext.Rip);
#else
						ProgramCounter = (void*)(ThreadContext.Pc);
#endif
					}
				}
			}

			// If we were able to suspend and capture the thread state
			if (ExceptionInformation.ContextRecord)
			{
				EXCEPTION_RECORD ExceptionRecord;
				ZeroMemory(&ExceptionRecord, sizeof(ExceptionRecord));
				ExceptionRecord.ExceptionCode = STILL_ACTIVE; // normally where EXCEPTION_ACCESS_VIOLATION would be

				// The thread context and exception recard are the two things expected
				ExceptionInformation.ExceptionRecord = &ExceptionRecord;

				(void)ReportContinuableEventUsingCrashReportClient(InType, &ExceptionInformation, ThreadHandle, InThreadId, ProgramCounter, ErrorMessage, IsInteractiveEnsureMode() ? EErrorReportUI::ShowDialog : EErrorReportUI::ReportInUnattendedMode);
			}

			if (SuspendCount != -1)
			{
				ResumeThread(ThreadHandle);
			}

			CloseHandle(ThreadHandle);
		}
	}
}

void ReportAssert(const TCHAR* ErrorMessage, void* ProgramCounter)
{
	/** This is the last place to gather memory stats before exception. */
	FGenericCrashContext::SetMemoryStats(FPlatformMemory::GetStats());

	FAssertInfo Info(ErrorMessage, ProgramCounter);

	ULONG_PTR Arguments[] = { (ULONG_PTR)&Info };
	if (FGenericPlatformMemory::bIsOOM)
	{
		::RaiseException(OutOfMemoryExceptionCode, 0, UE_ARRAY_COUNT(Arguments), Arguments);
	}
	else
	{
		::RaiseException(AssertExceptionCode, 0, UE_ARRAY_COUNT(Arguments), Arguments);
	}
}

FORCENOINLINE void ReportGPUCrash(const TCHAR* ErrorMessage, void* ProgramCounter)
{
	if (ProgramCounter == nullptr)
	{
		ProgramCounter = PLATFORM_RETURN_ADDRESS();
	}

	/** This is the last place to gather memory stats before exception. */
	FGenericCrashContext::SetMemoryStats(FPlatformMemory::GetStats());
	
	// GPUCrash can be called when the guarded entry is not set
#if !PLATFORM_SEH_EXCEPTIONS_DISABLED
	__try
	{
		FAssertInfo Info(ErrorMessage, ProgramCounter);

		ULONG_PTR Arguments[] = { (ULONG_PTR)&Info };
		::RaiseException(GPUCrashExceptionCode, 0, UE_ARRAY_COUNT(Arguments), Arguments);
	}
	__except (FPlatformMisc::GetCrashHandlingType() == ECrashHandlingType::Default ? ReportCrash(GetExceptionInformation()) : EXCEPTION_CONTINUE_SEARCH)	
	{
		FPlatformMisc::RequestExit(false, TEXT("WindowsPlatformCrashContext.ReportGPUCrash"));
	}
#endif
}

void ReportHang(const TCHAR* ErrorMessage, const uint64* StackFrames, int32 NumStackFrames, uint32 HungThreadId)
{
	if (ReportCrashCallCount > 0 || FDebug::HasAsserted())
	{
		// Don't report ensures after we've crashed/asserted, they simply may be a result of the crash as
		// the engine is already in a bad state.
		return;
	}

	FWindowsPlatformCrashContext CrashContext(ECrashContextType::Hang, ErrorMessage);
	CrashContext.SetCrashedProcess(FProcHandle(::GetCurrentProcess()));
	CrashContext.SetCrashedThreadId(HungThreadId);
	CrashContext.SetPortableCallStack(StackFrames, NumStackFrames);
	CrashContext.CaptureAllThreadContexts();

	EErrorReportUI ReportUI = IsInteractiveEnsureMode() ? EErrorReportUI::ShowDialog : EErrorReportUI::ReportInUnattendedMode;
	ReportCrashUsingCrashReportClient(CrashContext, nullptr, ReportUI);
}

// #CrashReport: 2015-05-28 This should be named EngineEnsureHandler
/**
 * Report an ensure to the crash reporting system
 */
void ReportEnsure(const TCHAR* ErrorMessage, void* ProgramCounter)
{
	ReportEvent(ECrashContextType::Ensure, ErrorMessage, FPlatformTLS::GetCurrentThreadId(), ProgramCounter);
}

/**
 * Report a hitch to the crash reporting system
 */
FORCENOINLINE void ReportStall(const TCHAR* ErrorMessage, uint32 HitchThreadId)
{
	ReportEvent(ECrashContextType::Stall, ErrorMessage, HitchThreadId, nullptr);
}

#if !IS_PROGRAM && 0
namespace {
	/** Utility class to test crashes during static initialization. */
	struct StaticInitCrasher
	{
		StaticInitCrasher()
		{
			// Test stack overflow
			//StackOverlowMe(0);

			// Test GPF
			//int* i = nullptr;
			//*i = 3;

			// Check assert (shouldn't work during static init)
			//check(false);
		}

		void StackOverlowMe(uint32 s) {
			uint32 buffer[1024];
			memset(&buffer, 0xdeadbeef, 1024);
			(void*)buffer;
			if (s == 0xffffffff)
				return;
			StackOverlowMe(s + 1);
		}
	};

	static StaticInitCrasher GCrasher;
}
#endif



#ifdef WINDOWS_PLATFORM_TYPES_GUARD
	static_assert(false, "Missing HideWindowsPlatformTypes.h in " __FILE__ );
#endif
