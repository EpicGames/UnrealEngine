// Copyright Epic Games, Inc. All Rights Reserved.

#include "Windows/WindowsPlatformFile.h"

#include "Windows/WindowsPlatformFilePrivate.h"

#include "Algo/Replace.h"
#include "Async/AsyncFileHandle.h"
#include "Async/AsyncWork.h"
#include "Async/MappedFileHandle.h"
#include "Containers/LockFreeList.h"
#include "Containers/UnrealString.h"
#include "CoreGlobals.h"
#include "CoreTypes.h"
#include "HAL/IConsoleManager.h"
#include "HAL/IPlatformFileManagedStorageWrapper.h"
#include "HAL/UnrealMemory.h"
#include "Logging/LogMacros.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Paths.h"
#include "Misc/PathViews.h"
#include "Misc/ScopeLock.h"
#include "ProfilingDebugging/PlatformFileTrace.h"
#include "Templates/Function.h"
#include <sys/utime.h>

#include "Windows/AllowWindowsPlatformTypes.h" // Required immediately before MicrosoftAsyncIO.h and the block below it
#include "Microsoft/MicrosoftAsyncIO.h"

namespace FileConstants
{
uint32 WIN_INVALID_SET_FILE_POINTER = INVALID_SET_FILE_POINTER;

bool GEnableFileShareDeleteFlag = false;
static FAutoConsoleVariableRef CVarEnableFileShareDeleteFlag(
	TEXT("file.allowdeleteopenfiles"),
	GEnableFileShareDeleteFlag,
	TEXT("Sets the the FILE_SHARE_DELETE flag on all file open operations so that files can be replaced while open. Only supported in editor builds."),
	ECVF_Default
);

} // namespace FileConstants
#include "Windows/HideWindowsPlatformTypes.h"


static int32 GetFileShareDeleteFlagIfEnabled()
{
#if WITH_EDITOR
	if (FileConstants::GEnableFileShareDeleteFlag)
	{
		return FILE_SHARE_DELETE;
	}
#endif
	return 0;
}

TLockFreePointerListUnordered<void, PLATFORM_CACHE_LINE_SIZE> MicrosoftAsyncIOEventPool;
bool GTriggerFailedMicrosoftRead = false;

#if !UE_BUILD_SHIPPING
static void TriggerFailedMicrosoftRead(const TArray<FString>& Args)
{
	GTriggerFailedMicrosoftRead = true;
}

static FAutoConsoleCommand TriggerFailedMicrosoftReadCmd(
	TEXT("TriggerFailedWindowsRead"),
	TEXT("Tests low level IO errors on Windows"),
	FConsoleCommandWithArgsDelegate::CreateStatic(&TriggerFailedMicrosoftRead)
);
#endif

namespace FileConstants
{

/** Special suffix to access Alternate Data Stream "Zone.Identifier" for NTFS files */
constexpr FStringView ZoneIdentifierStreamSuffix(TEXTVIEW(":Zone.Identifier"));

} // namespace FileConstants

namespace UE::WindowsPlatformFile::Private
{

int32 UEDayOfWeekToWindowsSystemTimeDayOfWeek(const EDayOfWeek InDayOfWeek)
{
	switch (InDayOfWeek)
	{
		case EDayOfWeek::Monday:
			return 1;
		case EDayOfWeek::Tuesday:
			return 2;
		case EDayOfWeek::Wednesday:
			return 3;
		case EDayOfWeek::Thursday:
			return 4;
		case EDayOfWeek::Friday:
			return 5;
		case EDayOfWeek::Saturday:
			return 6;
		case EDayOfWeek::Sunday:
			return 0;
		default:
			break;
	}

	return 0;
}

FDateTime WindowsFileTimeToUEDateTime(const FILETIME& InFileTime)
{
	// This roundabout conversion clamps the precision of the returned time value to match that of time_t (1 second precision)
	// This avoids issues when sending files over the network via cook-on-the-fly
	SYSTEMTIME SysTime;
	if (FileTimeToSystemTime(&InFileTime, &SysTime))
	{
		return FDateTime(SysTime.wYear, SysTime.wMonth, SysTime.wDay, SysTime.wHour, SysTime.wMinute, SysTime.wSecond);
	}

	// Failed to convert
	return FDateTime::MinValue();
}

FILETIME UEDateTimeToWindowsFileTime(const FDateTime& InDateTime)
{
	// This roundabout conversion clamps the precision of the returned time value to match that of time_t (1 second precision)
	// This avoids issues when sending files over the network via cook-on-the-fly
	SYSTEMTIME SysTime;
	SysTime.wYear = (WORD)InDateTime.GetYear();
	SysTime.wMonth = (WORD)InDateTime.GetMonth();
	SysTime.wDay = (WORD)InDateTime.GetDay();
	SysTime.wDayOfWeek = (WORD)UE::WindowsPlatformFile::Private::UEDayOfWeekToWindowsSystemTimeDayOfWeek(InDateTime.GetDayOfWeek());
	SysTime.wHour = (WORD)InDateTime.GetHour();
	SysTime.wMinute = (WORD)InDateTime.GetMinute();
	SysTime.wSecond = (WORD)InDateTime.GetSecond();
	SysTime.wMilliseconds = 0;

	FILETIME FileTime;
	SystemTimeToFileTime(&SysTime, &FileTime);

	return FileTime;
}

void UpdateOverlappedPos(OVERLAPPED& Overlapped, int64 UpdatedPos)
{
	ULARGE_INTEGER LI;
	LI.QuadPart = UpdatedPos;

	Overlapped.Offset = LI.LowPart;
	Overlapped.OffsetHigh = LI.HighPart;
}

} // namespace UE::WindowsPlatformFile::Private

#if USE_OVERLAPPED_IO

/**
 * This file reader uses overlapped i/o and double buffering to asynchronously read from files
 */
class FAsyncBufferedFileReaderWindows : public IFileHandle
{
protected:
	enum {DEFAULT_BUFFER_SIZE = 64 * 1024};
	/**
	 * The file handle to operate on
	 */
	HANDLE Handle;
	/**
	 * The size of the file that is being read
	 */
	int64 FileSize;
	/**
	 * Overall position in the file and buffers combined
	 */
	int64 FilePos;
	/**
	 * Overall position in the file as the OverlappedIO struct understands it
	 */
	uint64 OverlappedFilePos;
	/**
	 * These are the two buffers used for reading the file asynchronously
	 */
	int8* Buffers[2];
	/**
	 * The size of the buffers in bytes
	 */
	const int32 BufferSize;
	/**
	 * The current index of the buffer that we are serializing from
	 */
	int32 SerializeBuffer;
	/**
	 * The current index of the streaming buffer for async reading into
	 */
	int32 StreamBuffer;
	/**
	 * Where we are in the serialize buffer
	 */
	int32 SerializePos;
	/**
	 * Tracks which buffer has the async read outstanding (0 = first read after create/seek, 1 = streaming buffer)
	 */
	int32 CurrentAsyncReadBuffer;
	/**
	 * Desired access as passed to the Windows API when opening the file handle.  To be used in ShrinkBuffers to re-open the file handle.
	 */
	uint32 DesiredAccess;
	/**
	 * Share mode as passed to the Windows API when opening the file handle.  To be used in ShrinkBuffers to re-open the file handle.
	 */
	uint32 ShareMode;
	/**
	 * Flags as passed to the Windows API when opening the file handle.  To be used in ShrinkBuffers to re-open the file handle.
	 * NOTE: This is constrained to a subset of flags/attributes as noted on the ReOpenFile Windows API documentation.
	 */
	uint32 Flags;
	/**
	 * The overlapped IO struct to use for determining async state
	 */
	OVERLAPPED OverlappedIO;
	/**
	 * Used to track whether the last read reached the end of the file or not. Reset when a Seek happens
	 */
	bool bIsAtEOF;
	/**
	 * Whether there's a read outstanding or not
	 */
	bool bHasReadOutstanding;

	/**
	 * Closes the file handle
	 */
	bool Close(void)
	{
		if (Handle != nullptr)
		{
			// Close the file handle
			TRACE_PLATFORMFILE_BEGIN_CLOSE(Handle);
			BOOL CloseResult = CloseHandle(Handle);
#if PLATFORMFILETRACE_ENABLED
			if (CloseResult)
			{
				TRACE_PLATFORMFILE_END_CLOSE(Handle);
			}
			else
			{
				TRACE_PLATFORMFILE_FAIL_CLOSE(Handle);
			}
#else
			(void)CloseResult;
#endif
			Handle = nullptr;
		}
		return true;
	}

	/**
	 * This toggles the buffers we read into & serialize out of between buffer indices 0 & 1
	 */
	FORCEINLINE void SwapBuffers()
	{
		StreamBuffer ^= 1;
		SerializeBuffer ^= 1;
		// We are now at the beginning of the serialize buffer
		SerializePos = 0;
	}

	FORCEINLINE void CopyOverlappedPosition()
	{
		ULARGE_INTEGER LI;
		LI.QuadPart = OverlappedFilePos;
		OverlappedIO.Offset = LI.LowPart;
		OverlappedIO.OffsetHigh = LI.HighPart;
	}

	FORCEINLINE void UpdateFileOffsetAfterRead(uint32 AmountRead)
	{
		bHasReadOutstanding = false;
		OverlappedFilePos += AmountRead;
		// Update the overlapped structure since it uses this for where to read from
		CopyOverlappedPosition();
		if (OverlappedFilePos >= uint64(FileSize))
		{
			bIsAtEOF = true;
		}
	}

	bool WaitForAsyncRead()
	{
		// Check for already being at EOF because we won't issue a read
		if (bIsAtEOF || !bHasReadOutstanding)
		{
			return true;
		}
		uint32 NumRead = 0;
		if (GetOverlappedResult(Handle, &OverlappedIO, (::DWORD*)&NumRead, true) != false)
		{
			TRACE_PLATFORMFILE_END_READ(&OverlappedIO, NumRead);
			UpdateFileOffsetAfterRead(NumRead);
			return true;
		}
		else if (GetLastError() == ERROR_HANDLE_EOF)
		{
			TRACE_PLATFORMFILE_END_READ(&OverlappedIO, 0);
			bIsAtEOF = true;
			return true;
		}
		return false;
	}

	void StartAsyncRead(int32 BufferToReadInto)
	{
		if (!bIsAtEOF)
		{
			bHasReadOutstanding = true;
			CurrentAsyncReadBuffer = BufferToReadInto;
			uint32 NumRead = 0;
			// Now kick off an async read
			TRACE_PLATFORMFILE_BEGIN_READ(&OverlappedIO, Handle, OverlappedFilePos, BufferSize);
			if (!ReadFile(Handle, Buffers[BufferToReadInto], BufferSize, (::DWORD*)&NumRead, &OverlappedIO))
			{
				uint32 ErrorCode = GetLastError();
				if (ErrorCode != ERROR_IO_PENDING)
				{
					TRACE_PLATFORMFILE_END_READ(&OverlappedIO, 0);
					bIsAtEOF = true;
					bHasReadOutstanding = false;
				}
			}
			else
			{
				// Read completed immediately
				TRACE_PLATFORMFILE_END_READ(&OverlappedIO, NumRead);
				UpdateFileOffsetAfterRead(NumRead);
			}
		}
	}

	FORCEINLINE void StartStreamBufferRead()
	{
		StartAsyncRead(StreamBuffer);
	}

	FORCEINLINE void StartSerializeBufferRead()
	{
		StartAsyncRead(SerializeBuffer);
	}

	FORCEINLINE bool IsValid()
	{
		return Handle != nullptr && Handle != INVALID_HANDLE_VALUE;
	}

public:
	FAsyncBufferedFileReaderWindows(HANDLE InHandle, uint32 InDesiredAccess, uint32 InShareMode, uint32 InFlags, int32 InBufferSize = DEFAULT_BUFFER_SIZE) :
		Handle(InHandle),
		FilePos(0),
		OverlappedFilePos(0),
		BufferSize(InBufferSize),
		SerializeBuffer(0),
		StreamBuffer(1),
		SerializePos(0),
		CurrentAsyncReadBuffer(0),
		DesiredAccess(InDesiredAccess),
		ShareMode(InShareMode),
		Flags(InFlags),
		bIsAtEOF(false),
		bHasReadOutstanding(false)
	{
		LARGE_INTEGER LI;
		GetFileSizeEx(Handle, &LI);
		FileSize = LI.QuadPart;
		// Allocate our two buffers
		Buffers[0] = (int8*)FMemory::Malloc(BufferSize);
		Buffers[1] = (int8*)FMemory::Malloc(BufferSize);

		// Zero the overlapped structure
		FMemory::Memzero(&OverlappedIO, sizeof(OVERLAPPED));

		// Kick off the first async read
		StartSerializeBufferRead();
	}

	virtual ~FAsyncBufferedFileReaderWindows(void)
	{
		// Can't free the buffers or close the file if a read is outstanding
		WaitForAsyncRead();
		Close();
		FMemory::Free(Buffers[0]);
		FMemory::Free(Buffers[1]);
	}

	virtual bool Seek(int64 InPos) override
	{
		check(IsValid());
		check(InPos >= 0);
		check(InPos <= FileSize);

		// Determine the change in locations
		int64 PosDelta = InPos - FilePos;
		if (PosDelta == 0)
		{
			// Same place so no work to do
			return true;
		}

		// No matter what, we need to wait for the current async read to finish since we most likely need to issue a new one
		if (!WaitForAsyncRead())
		{
			return false;
		}

		FilePos = InPos;

		// If the requested location is not within our current serialize buffer, we need to start the whole read process over
		bool bWithinSerializeBuffer = (PosDelta < 0 && (SerializePos - FMath::Abs(PosDelta) >= 0)) ||
			(PosDelta > 0 && ((PosDelta + SerializePos) < BufferSize));
		if (bWithinSerializeBuffer)
		{
			// Still within the serialize buffer so just update the position
			SerializePos += (int32)PosDelta;
		}
		else
		{
			// Reset our EOF tracking and let the read handle setting it if need be
			bIsAtEOF = false;
			// Not within our buffer so start a new async read on the serialize buffer
			OverlappedFilePos = InPos;
			CopyOverlappedPosition();
			CurrentAsyncReadBuffer = SerializeBuffer;
			SerializePos = 0;
			StartSerializeBufferRead();
		}
		return true;
	}

	virtual bool SeekFromEnd(int64 NewPositionRelativeToEnd = 0) override
	{
		check(IsValid());
		check(NewPositionRelativeToEnd <= 0);

		// Position is negative so this is actually subtracting
		return Seek(FileSize + NewPositionRelativeToEnd);
	}

	virtual int64 Tell(void) override
	{
		check(IsValid());
		return FilePos;
	}

	virtual int64 Size() override
	{
		check(IsValid());
		return FileSize;
	}

	virtual bool Read(uint8* Dest, int64 BytesToRead) override
	{
		check(IsValid());
		// If zero were requested, quit (some calls like to do zero sized reads)
		if (BytesToRead <= 0)
		{
			return false;
		}

		if (CurrentAsyncReadBuffer == SerializeBuffer)
		{
			// First async read after either construction or a seek
			if (!WaitForAsyncRead())
			{
				return false;
			}
			StartStreamBufferRead();
		}

		check(Dest != nullptr)
		// While there is data to copy
		while (BytesToRead > 0)
		{
			// Figure out how many bytes we can read from the serialize buffer
			int64 NumToCopy = FMath::Min<int64>(BytesToRead, BufferSize - SerializePos);
			if (FilePos + NumToCopy > FileSize)
			{
				// Tried to read past the end of the file, so fail
				return false;
			}
			// See if we are at the end of the serialize buffer or not
			if (NumToCopy > 0)
			{
				FMemory::Memcpy(Dest, &Buffers[SerializeBuffer][SerializePos], NumToCopy);

				// Update the internal positions
				SerializePos += (int32)NumToCopy;
				check(SerializePos <= BufferSize);
				FilePos += NumToCopy;
				check(FilePos <= FileSize);

				// Decrement the number of bytes we copied
				BytesToRead -= NumToCopy;

				// Now offset the dest pointer with the chunk we copied
				Dest = (uint8*)Dest + NumToCopy;
			}
			else
			{
				// We've crossed the buffer boundary and now need to make sure the stream buffer read is done
				if (!WaitForAsyncRead())
				{
					return false;
				}
				SwapBuffers();
				StartStreamBufferRead();
			}
		}
		return true;
	}

	virtual bool ReadAt(uint8* Destination, int64 BytesToRead, int64 Offset) override
	{
		if (BytesToRead < 0 || Offset < 0 || (BytesToRead + Offset) > FileSize)
		{
			return false;
		}

		if (BytesToRead == 0)
		{
			return true;
		}

		OVERLAPPED LocalOverlappedIO;
		FMemory::Memzero(&LocalOverlappedIO, sizeof(OVERLAPPED));

		// Calls to ::ReadAt should be rare enough that we can create/destroy events for each read as needed
		LocalOverlappedIO.hEvent = CreateEvent(NULL, 0, 0, NULL);

		if (LocalOverlappedIO.hEvent == NULL)
		{
			// TODO Error
			return false;
		}

		ON_SCOPE_EXIT
		{
			CloseHandle(LocalOverlappedIO.hEvent);
		};

		TRACE_PLATFORMFILE_BEGIN_READ(&LocalOverlappedIO, Handle, Offset, BytesToRead);
		int64 TotalNumRead = 0;
		do
		{
			uint32 BytesToRead32 = (uint32)FMath::Min<int64>(BytesToRead, FWindowsPlatformMisc::IsWine() ? int64(0x7ffff000) : int64(UINT32_MAX));
			uint32 NumRead = 0;

			UE::WindowsPlatformFile::Private::UpdateOverlappedPos(LocalOverlappedIO, Offset);

			if (!ReadFile(Handle, Destination, BytesToRead32, (::DWORD*)&NumRead, &LocalOverlappedIO))
			{
				if (GetLastError() != ERROR_IO_PENDING)
				{
					// TODO Error
					TRACE_PLATFORMFILE_END_READ(&LocalOverlappedIO, TotalNumRead);
					return false;
				}

				if (!GetOverlappedResult(Handle, &LocalOverlappedIO, (::DWORD*)&NumRead, true))
				{
					// TODO Error
					TRACE_PLATFORMFILE_END_READ(&LocalOverlappedIO, TotalNumRead);
					return false;
				}
			}

			BytesToRead -= NumRead;
			TotalNumRead += NumRead;
			Destination += NumRead;

			if (NumRead == 0)
			{
				// Reading less than requested bytes can occur due to an internal limit, but we always expect
				// at least some bytes to be read. If 0 bytes were read repeatedly, we could deadlock. So return
				// false when NumRead is 0 rather than continuing to try.
				// Pick an error we can use to report to the caller why we failed. We arbitrarily use
				// ERROR_CAN_NOT_COMPLETE, since it seems like one that ReadFile does not normally return.
				SetLastError(ERROR_CAN_NOT_COMPLETE);
				TRACE_PLATFORMFILE_END_READ(&LocalOverlappedIO, TotalNumRead);
				return false;
			}
		} while (BytesToRead > 0);

		TRACE_PLATFORMFILE_END_READ(&LocalOverlappedIO, TotalNumRead);
		return true;
	}

	virtual bool Write(const uint8* Source, int64 BytesToWrite) override
	{
		check(0 && "This is an async reader only and doesn't support writing");
		return false;
	}

	virtual bool Flush(const bool bFullFlush = false) override
	{
		// Reader only, so don't need to support flushing
		return false;
	}

	virtual bool Truncate(int64 NewSize) override
	{
		// Reader only, so don't need to support truncation
		return false;
	}

	virtual void ShrinkBuffers() override
	{
		if (IsValid())
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(ShrinkBuffers);

			TRACE_PLATFORMFILE_BEGIN_REOPEN(Handle);
			HANDLE NewFileHandle = ReOpenFile(Handle, DesiredAccess, ShareMode, Flags);
			TRACE_PLATFORMFILE_END_REOPEN(NewFileHandle);
			if (NewFileHandle == INVALID_HANDLE_VALUE)
			{
				// We are not allowed to change this->IsValid() from true to false.
				// If the Reopen fails, keep the old FileHandle
				return;
			}

			TRACE_PLATFORMFILE_BEGIN_CLOSE(Handle);
			BOOL CloseResult = CloseHandle(Handle);
#if PLATFORMFILETRACE_ENABLED
			if (CloseResult)
			{
				TRACE_PLATFORMFILE_END_CLOSE(Handle);
			}
			else
			{
				TRACE_PLATFORMFILE_FAIL_CLOSE(Handle);
			}
#else
			(void)CloseResult;
#endif
			Handle = NewFileHandle;
			check(IsValid());
		}
	}
};

#endif // USE_OVERLAPPED_IO

/** 
 * Windows file handle implementation
**/

class FFileHandleWindows : public IFileHandle
{
	enum { READWRITE_SIZE = 1024 * 1024 };
	HANDLE FileHandle;
	/** The overlapped IO struct to use for determining async state	*/
	OVERLAPPED OverlappedIO;
	/** Manages the location of our file position for setting on the overlapped struct for reads/writes */
	int64 FilePos;
	/** Need the file size for seek from end */
	int64 FileSize;
	/** Desired access as passed to the Windows API when opening the file handle.  To be used in ShrinkBuffers to re-open the file handle. */
	uint32 DesiredAccess;
	/** Share mode as passed to the Windows API when opening the file handle.  To be used in ShrinkBuffers to re-open the file handle. */
	uint32 ShareMode;
	/**
	 * Flags as passed to the Windows API when opening the file handle.  To be used in ShrinkBuffers to re-open the file handle.
	 * NOTE: This is constrained to a subset of flags/attributes as noted on the ReOpenFile Windows API documentation.
	 */
	uint32 Flags;

	FORCEINLINE bool UpdatedNonOverlappedPos()
	{
		LARGE_INTEGER LI;
		LI.QuadPart = FilePos;
		return SetFilePointer(FileHandle, LI.LowPart, &LI.HighPart, FILE_BEGIN) != INVALID_SET_FILE_POINTER;
	}

	void UpdateFileSize()
	{
		LARGE_INTEGER LI;
		BOOL Success = GetFileSizeEx(FileHandle, &LI);
		if (Success)
		{
			FileSize = LI.QuadPart;
		}
		else
		{
			// This is a rare condition but it has been observed in the wild
			// for SMB file shares. Since it's important to know the file size
			// we render the file handle unusable if this condition occurs
			FileSize = -1;

			UE_LOGF(LogTemp, Warning, "GetFileSizeEx: Failed to get file size for handle!");
		}
	}

public:
	FFileHandleWindows(HANDLE InFileHandle, uint32 InDesiredAccess, uint32 InShareMode, uint32 InFlags)
		: FileHandle(InFileHandle)
		, FilePos(0)
		, FileSize(0)
		, DesiredAccess(InDesiredAccess)
		, ShareMode(InShareMode)
		, Flags(InFlags)
	{
		if (IsValid())
		{
			UpdateFileSize();
		}
		FMemory::Memzero(&OverlappedIO, sizeof(OVERLAPPED));
	}
	virtual ~FFileHandleWindows()
	{
		TRACE_PLATFORMFILE_BEGIN_CLOSE(FileHandle);
		BOOL CloseResult = CloseHandle(FileHandle);
#if PLATFORMFILETRACE_ENABLED
		if (CloseResult)
		{
			TRACE_PLATFORMFILE_END_CLOSE(FileHandle);
		}
		else
		{
			TRACE_PLATFORMFILE_FAIL_CLOSE(FileHandle);
		}
#else
		(void)CloseResult;
#endif
		FileHandle = NULL;
	}
	bool IsValid() const
	{
		return FileHandle != NULL && FileHandle != INVALID_HANDLE_VALUE && FileSize != -1;
	}
	virtual int64 Tell(void) override
	{
		check(IsValid());
		return FilePos;
	}
	virtual int64 Size() override
	{
		check(IsValid());
		return FileSize;
	}
	virtual bool Seek(int64 NewPosition) override
	{
		check(IsValid());
		check(NewPosition >= 0);

		FilePos = NewPosition;
		UE::WindowsPlatformFile::Private::UpdateOverlappedPos(OverlappedIO, FilePos);
		return true;
	}
	virtual bool SeekFromEnd(int64 NewPositionRelativeToEnd = 0) override
	{
		check(IsValid());
		check(NewPositionRelativeToEnd <= 0);

		// Position is negative so this is actually subtracting
		return Seek(FileSize + NewPositionRelativeToEnd);
	}
	virtual bool Read(uint8* Destination, int64 BytesToRead) override
	{
		check(IsValid());
		// Now kick off an async read
		TRACE_PLATFORMFILE_BEGIN_READ(&OverlappedIO, FileHandle, FilePos, BytesToRead);

		int64 TotalNumRead = 0;
		bool bAtEndOfFile = FilePos >= FileSize;
		if (bAtEndOfFile)
		{
			return false;
		}

		do
		{
			uint32 BytesToRead32 = (uint32)FMath::Min<int64>(BytesToRead, FWindowsPlatformMisc::IsWine() ? int64(0x7ffff000) : int64(UINT32_MAX));
			uint32 NumRead = 0;

			if (!ReadFile(FileHandle, Destination, BytesToRead32, (::DWORD*)&NumRead, &OverlappedIO))
			{
				uint32 ErrorCode = GetLastError();
				if (ErrorCode != ERROR_IO_PENDING)
				{
					// Read failed
					TRACE_PLATFORMFILE_END_READ(&OverlappedIO, 0);
					return false;
				}
				// Wait for the read to complete
				NumRead = 0;
				if (!GetOverlappedResult(FileHandle, &OverlappedIO, (::DWORD*)&NumRead, true))
				{
					// Read failed
					TRACE_PLATFORMFILE_END_READ(&OverlappedIO, 0);
					return false;
				}
			}

			BytesToRead -= NumRead;
			TotalNumRead += NumRead;
			Destination += NumRead;
			// Update where we are in the file
			FilePos += NumRead;
			bAtEndOfFile = FilePos >= FileSize;
			UE::WindowsPlatformFile::Private::UpdateOverlappedPos(OverlappedIO, FilePos);

			if (NumRead == 0 && BytesToRead32 != 0)
			{
				// Reading less than requested bytes can occur due to an internal limit, but we always expect
				// at least some bytes to be read. If 0 bytes were read repeatedly, we could deadlock. So return
				// false when NumRead is 0 rather than continuing to try.
				// Pick an error we can use to report to the caller why we failed. We arbitrarily use
				// ERROR_CAN_NOT_COMPLETE, since it seems like one that ReadFile does not normally return.
				SetLastError(ERROR_CAN_NOT_COMPLETE);
				TRACE_PLATFORMFILE_END_READ(&OverlappedIO, TotalNumRead);
				return false;
			}
		} while (BytesToRead > 0 && !bAtEndOfFile);
		TRACE_PLATFORMFILE_END_READ(&OverlappedIO, TotalNumRead);
		return true;
	}
	virtual bool ReadAt(uint8* Destination, int64 BytesToRead, int64 Offset) override
	{
		check(IsValid());

		if (BytesToRead < 0 || Offset < 0 || (BytesToRead + Offset) > FileSize)
		{
			return false;
		}

		if (BytesToRead == 0)
		{
			return true;
		}

		OVERLAPPED LocalOverlappedIO;
		FMemory::Memzero(&LocalOverlappedIO, sizeof(OVERLAPPED));

		TRACE_PLATFORMFILE_BEGIN_READ(&LocalOverlappedIO, FileHandle, Offset, BytesToRead);
		int64 TotalNumRead = 0;
		do
		{
			uint32 BytesToRead32 = (uint32)FMath::Min<int64>(BytesToRead, FWindowsPlatformMisc::IsWine() ? int64(0x7ffff000) : int64(UINT32_MAX));
			
			uint32 NumRead = 0;

			UE::WindowsPlatformFile::Private::UpdateOverlappedPos(LocalOverlappedIO, Offset);

			if (!ReadFile(FileHandle, Destination, BytesToRead32, (::DWORD*)&NumRead, &LocalOverlappedIO))
			{
				TRACE_PLATFORMFILE_END_READ(&LocalOverlappedIO, TotalNumRead);
				return false;
			}

			BytesToRead -= NumRead;
			TotalNumRead += NumRead;
			Destination += NumRead;

			if (NumRead == 0)
			{
				// Reading less than requested bytes can occur due to an internal limit, but we always expect
				// at least some bytes to be read. If 0 bytes were read repeatedly, we could deadlock. So return
				// false when NumRead is 0 rather than continuing to try.
				// Pick an error we can use to report to the caller why we failed. We arbitrarily use
				// ERROR_CAN_NOT_COMPLETE, since it seems like one that ReadFile does not normally return.
				SetLastError(ERROR_CAN_NOT_COMPLETE);
				TRACE_PLATFORMFILE_END_READ(&OverlappedIO, TotalNumRead);
				return false;
			}
		} while (BytesToRead > 0);

		TRACE_PLATFORMFILE_END_READ(&LocalOverlappedIO, TotalNumRead);
		return true;
	}
	virtual bool Write(const uint8* Source, int64 BytesToWrite) override
	{
		check(IsValid());

		TRACE_PLATFORMFILE_BEGIN_WRITE(this, FileHandle, FilePos, BytesToWrite);

		int64 TotalNumWritten = 0;
		do
		{
			uint32 BytesToWrite32 = (uint32)FMath::Min<int64>(BytesToWrite, FWindowsPlatformMisc::IsWine() ? int64(0x7ffff000) : int64(UINT32_MAX));
			uint32 NumWritten = 0;
			// Now kick off an async write
			if (!WriteFile(FileHandle, Source, BytesToWrite32, (::DWORD*)&NumWritten, &OverlappedIO))
			{
				uint32 ErrorCode = GetLastError();
				if (ErrorCode != ERROR_IO_PENDING)
				{
					// Write failed
					TRACE_PLATFORMFILE_END_WRITE(this, 0);
					return false;
				}
				// Wait for the write to complete
				NumWritten = 0;
				if (!GetOverlappedResult(FileHandle, &OverlappedIO, (::DWORD*)&NumWritten, true))
				{
					// Write failed
					TRACE_PLATFORMFILE_END_WRITE(this, 0);
					return false;
				}
			}

			BytesToWrite -= NumWritten;
			Source += NumWritten;
			TotalNumWritten += NumWritten;
			// Update where we are in the file
			FilePos += NumWritten;
			UE::WindowsPlatformFile::Private::UpdateOverlappedPos(OverlappedIO, FilePos);
			FileSize = FMath::Max(FilePos, FileSize);

			if (NumWritten == 0 && BytesToWrite32 != 0)
			{
				// Writing less than requested bytes can occur due to an internal limit, but we always expect
				// at least some bytes to be written. If 0 bytes were written repeatedly, we could deadlock. So return
				// false when NumWritten is 0 rather than continuing to try.
				// Pick an error we can use to report to the caller why we failed. We arbitrarily use
				// ERROR_CAN_NOT_COMPLETE, since it seems like one that WriteFile does not normally return.
				SetLastError(ERROR_CAN_NOT_COMPLETE);
				TRACE_PLATFORMFILE_END_WRITE(this, TotalNumWritten);
				return false;
			}
		} while (BytesToWrite > 0);

		TRACE_PLATFORMFILE_END_WRITE(this, TotalNumWritten);
		return true;
	}
	virtual bool Flush(const bool bFullFlush = false) override
	{
		check(IsValid());
		return FlushFileBuffers(FileHandle) != 0;
	}
	virtual bool Truncate(int64 NewSize) override
	{
		// SetEndOfFile isn't an overlapped operation, so we need to call UpdatedNonOverlappedPos after seeking to ensure that the file pointer is in the correct place
		check(IsValid());
		if (Seek(NewSize) && UpdatedNonOverlappedPos() && SetEndOfFile(FileHandle) != 0)
		{
			UpdateFileSize();
			check(IsValid());
			return true;
		}
		return false;
	}
	virtual void ShrinkBuffers() override
	{
		if (IsValid())
		{
			TRACE_PLATFORMFILE_BEGIN_REOPEN(FileHandle);
			HANDLE NewFileHandle = ReOpenFile(FileHandle, DesiredAccess, ShareMode, Flags);
			TRACE_PLATFORMFILE_END_REOPEN(NewFileHandle);
			if (NewFileHandle == INVALID_HANDLE_VALUE)
			{
				// We are not allowed to change this->IsValid() from true to false.
				// If the Reopen fails, keep the old FileHandle
				return;
			}

			TRACE_PLATFORMFILE_BEGIN_CLOSE(FileHandle);
			BOOL CloseResult = CloseHandle(FileHandle);
#if PLATFORMFILETRACE_ENABLED
			if (CloseResult)
			{
				TRACE_PLATFORMFILE_END_CLOSE(FileHandle);
			}
			else
			{
				TRACE_PLATFORMFILE_FAIL_CLOSE(FileHandle);
			}
#else
			(void)CloseResult;
#endif
			FileHandle = NewFileHandle;
			check(IsValid());
		}
	}
};

class FMappedFileRegionWindows final : public IMappedFileRegion
{
	class FMappedFileHandleWindows* Parent;
	const uint8* AlignedMappedPtr;
	size_t AlignedMappedSize;
public:
	FMappedFileRegionWindows(const uint8* InMappedPtr, const uint8* InAlignedMappedPtr, size_t InMappedSize, size_t InAlignedMappedSize, const FString& InDebugFilename, size_t InDebugOffsetRelativeToFile, class FMappedFileHandleWindows* InParent)
		: IMappedFileRegion(InMappedPtr, InMappedSize, InDebugFilename, InDebugOffsetRelativeToFile)
		, Parent(InParent)
		, AlignedMappedPtr(InAlignedMappedPtr)
		, AlignedMappedSize(InAlignedMappedSize)
	{
	}

	~FMappedFileRegionWindows();

	virtual void PreloadHint(int64 PreloadOffset = 0, int64 BytesToPreload = MAX_int64) override
	{
		if (PreloadOffset < GetMappedSize())
		{
			SIZE_T PageSize = FWindowsPlatformMemory::GetConstants().PageSize;
			int64 SizeAfterOffset = GetMappedSize() - PreloadOffset;
			int64 PreloadSize = FMath::Min(SizeAfterOffset, BytesToPreload);

			for (const uint8* It = GetMappedPtr() + PreloadOffset, *End = It + PreloadSize; It < End; It += PageSize)
			{
				volatile uint8 Foo = *It;
			}

			// Once we drop Win7 support we can try this Win8 API instead
			// WIN32_MEMORY_RANGE_ENTRY Range = { GetMappedPtr() + PreloadOffset, PreloadSize };
			// PrefetchVirtualMemory(GetCurrentProcess(), 1, &Range, 0);
		}
	}
};

class FMappedFileHandleWindows final : public IMappedFileHandle
{
	HANDLE Handle;
	HANDLE MappingHandle;
	FString DebugFilename;
	std::atomic<int32> NumOutstandingRegions;
public:
	FMappedFileHandleWindows(HANDLE InHandle, HANDLE InMappingHandle, int64 Size, const TCHAR* InDebugFilename)
		: IMappedFileHandle(Size)
		, Handle(InHandle)
		, MappingHandle(InMappingHandle)
		, DebugFilename(InDebugFilename)
		, NumOutstandingRegions(0)
	{
		check(Size >= 0);
		check(Handle != INVALID_HANDLE_VALUE);
	}
	~FMappedFileHandleWindows()
	{
		// can't delete the file before you delete all outstanding regions
#if UE_BUILD_SHIPPING
		UE_CLOGF(NumOutstandingRegions.load(std::memory_order_relaxed), LogHAL, Error, "Cleaning mapped file with alive mapped regions: %ls", *DebugFilename);
#else
		UE_CLOGF(NumOutstandingRegions.load(std::memory_order_relaxed), LogHAL, Fatal, "Cleaning mapped file with alive mapped regions: %ls", *DebugFilename);
#endif

		CloseHandle(MappingHandle);
		TRACE_PLATFORMFILE_BEGIN_CLOSE(Handle);
		BOOL CloseResult = CloseHandle(Handle);
#if PLATFORMFILETRACE_ENABLED
		if (CloseResult)
		{
			TRACE_PLATFORMFILE_END_CLOSE(Handle);
		}
		else
		{
			TRACE_PLATFORMFILE_FAIL_CLOSE(Handle);
		}
#else
		(void)CloseResult;
#endif
	}
	virtual IMappedFileRegion* MapRegion(int64 Offset = 0, int64 BytesToMap = MAX_int64, FFileMappingFlags Flags = EMappedFileFlags::ENone) override
	{
		check(Offset < GetFileSize()); // don't map zero bytes and don't map off the end of the file
		BytesToMap = FMath::Min<int64>(BytesToMap, GetFileSize() - Offset);
		check(BytesToMap > 0); // don't map zero bytes

		DWORD OpenMappingAccess = FILE_MAP_READ;
		if (EnumHasAnyFlags(Flags.Flags, EMappedFileFlags::EFileWritable))
		{
			OpenMappingAccess |= FILE_MAP_WRITE;
		}

		SIZE_T VirtualMemoryGranularity = FWindowsPlatformMemory::GetConstants().OsAllocationGranularity;
		int64 AlignedOffset = AlignDown(Offset, VirtualMemoryGranularity);
		int64 AlignedSize = Align(BytesToMap + Offset - AlignedOffset, VirtualMemoryGranularity);

		ULARGE_INTEGER LI;
		LI.QuadPart = AlignedOffset;

		const uint8* AlignedMapPtr = (const uint8*)MapViewOfFile(MappingHandle, OpenMappingAccess, LI.HighPart, LI.LowPart, AlignedSize + AlignedOffset > GetFileSize() ? 0 : AlignedSize);
		if (!AlignedMapPtr)
		{
			return nullptr;
		}
		const uint8* MapPtr = AlignedMapPtr + Offset - AlignedOffset;
		FMappedFileRegionWindows* Result = new FMappedFileRegionWindows(MapPtr, AlignedMapPtr, BytesToMap, AlignedSize, DebugFilename, Offset, this);
		NumOutstandingRegions.fetch_add(1, std::memory_order_relaxed);
		return Result;
	}

	void UnMap(FMappedFileRegionWindows* Region)
	{
		int32 OldNumOutstandingRegions = NumOutstandingRegions.fetch_sub(1, std::memory_order_relaxed);
		check(OldNumOutstandingRegions > 0);
	}
};

FMappedFileRegionWindows::~FMappedFileRegionWindows()
{
	UnmapViewOfFile((LPVOID)AlignedMappedPtr);
	Parent->UnMap(this);
}

void FWindowsPlatformFile::NormalizeWindowsPath(FStringBuilderBase& Path, bool bIsFilename)
{
	FPathViews::ToAbsolutePathInline(Path);

	// NormalizeFilename was already called by ConvertRelativePathToFull, but we still need to do the extra steps in NormalizeDirectoryName if it is a directory
	if (!bIsFilename)
	{
		FPathViews::NormalizeDirectoryName(Path);
	}

	// Remove duplicate slashes
	const bool bIsUNCPath = Path.ToView().StartsWith(TEXTVIEW("//"));
		
	FPathViews::RemoveDuplicateSlashes(Path);

	if (bIsUNCPath)
	{
		// Keep // at the beginning.  If There are more than two / at the beginning, replace them with just //.
		Path.Prepend(TEXTVIEW("/"));
	}

	// We now have a canonical, strict-valid, absolute Unreal Path.  Convert it to a Windows Path.
	for (TCHAR& Char : TArrayView<TCHAR>(Path.GetData(), Path.Len()))
	{
		if (Char == TEXT('/'))
		{
			Char = TEXT('\\');
		}
	}

	// Handle Windows paths with null-terminated length over MAX_PATH
	if (Path.Len() >= MAX_PATH)
	{
		if (bIsUNCPath)
		{
			Path.ReplaceAt(0, 1, TEXTVIEW("\\\\?\\UNC"));
		}
		else
		{
			Path.Prepend(TEXTVIEW("\\\\?\\"));
		}
	}
}
	
FWindowsPlatformFile::FNormalizedFilename::FNormalizedFilename(const TCHAR* Filename)
{
	Append(Filename);
	NormalizeWindowsPath(*this, /* bIsFilename */ true);
}

FWindowsPlatformFile::FNormalizedFilename::FNormalizedFilename(FStringView Filename)
{
	Append(Filename);
	NormalizeWindowsPath(*this, /* bIsFilename */ true);
}

FWindowsPlatformFile::FNormalizedFilename::FNormalizedFilename(FStringView Dir, FStringView Filename)
{
	Append(Dir);
	FPathViews::Append(*this, Filename);
	NormalizeWindowsPath(*this, /* bIsFilename */ true);
}

FWindowsPlatformFile::FNormalizedDirectory::FNormalizedDirectory(const TCHAR* Directory)
{
	Append(Directory);
	NormalizeWindowsPath(*this, /* bIsFilename */ false);
}

bool FWindowsPlatformFile::FileExists(const TCHAR* Filename)
{
	uint32 Result = GetFileAttributesW(*FNormalizedFilename(Filename));
	if (Result != 0xFFFFFFFF && !(Result & FILE_ATTRIBUTE_DIRECTORY))
	{
		return true;
	}
	return false;
}

int64 FWindowsPlatformFile::FileSize(const TCHAR* Filename)
{
	WIN32_FILE_ATTRIBUTE_DATA Info;
	if (!!GetFileAttributesExW(*FNormalizedFilename(Filename), GetFileExInfoStandard, &Info))
	{
		if ((Info.dwFileAttributes & FILE_ATTRIBUTE_DIRECTORY) == 0)
		{
			LARGE_INTEGER li;
			li.HighPart = Info.nFileSizeHigh;
			li.LowPart = Info.nFileSizeLow;
			return li.QuadPart;
		}
	}
	return -1;
}

bool FWindowsPlatformFile::DeleteFile(const TCHAR* Filename)
{
	FNormalizedFilename NormalizedFilename(Filename);
	return !!DeleteFileW(*NormalizedFilename);
}

bool FWindowsPlatformFile::IsReadOnly(const TCHAR* Filename)
{
	uint32 Result = GetFileAttributesW(*FNormalizedFilename(Filename));
	if (Result != 0xFFFFFFFF)
	{
		return !!(Result & FILE_ATTRIBUTE_READONLY);
	}
	return false;
}

bool FWindowsPlatformFile::MoveFile(const TCHAR* To, const TCHAR* From)
{
	return !!MoveFileW(*FNormalizedFilename(From), *FNormalizedFilename(To));
}

bool FWindowsPlatformFile::SetReadOnly(const TCHAR* Filename, bool bNewReadOnlyValue)
{
	return !!SetFileAttributesW(*FNormalizedFilename(Filename), bNewReadOnlyValue ? FILE_ATTRIBUTE_READONLY : FILE_ATTRIBUTE_NORMAL);
}

FDateTime FWindowsPlatformFile::GetTimeStamp(const TCHAR* Filename)
{
	WIN32_FILE_ATTRIBUTE_DATA Info;
	if (GetFileAttributesExW(*FNormalizedFilename(Filename), GetFileExInfoStandard, &Info))
	{
		return UE::WindowsPlatformFile::Private::WindowsFileTimeToUEDateTime(Info.ftLastWriteTime);
	}

	return FDateTime::MinValue();
}

void FWindowsPlatformFile::SetTimeStamp(const TCHAR* Filename, FDateTime DateTime)
{
	TRACE_PLATFORMFILE_BEGIN_OPEN(Filename);
	HANDLE Handle = CreateFileW(*FNormalizedFilename(Filename), FILE_WRITE_ATTRIBUTES, FILE_SHARE_READ, nullptr, OPEN_EXISTING, FILE_ATTRIBUTE_NORMAL | FILE_FLAG_OVERLAPPED, nullptr);
	if (Handle != INVALID_HANDLE_VALUE)
	{
		TRACE_PLATFORMFILE_END_OPEN(Handle);
		const FILETIME ModificationFileTime = UE::WindowsPlatformFile::Private::UEDateTimeToWindowsFileTime(DateTime);
		SetFileTime(Handle, nullptr, nullptr, &ModificationFileTime);
		TRACE_PLATFORMFILE_BEGIN_CLOSE(Handle);
#if PLATFORMFILETRACE_ENABLED
		// MSVC static analysis has a rule that reports the argument to CloseHandle as uninitialized memory after the call to CloseHandle, so we have to save it ahead of time
		uint64 SavedHandle = uint64(Handle);
#endif
		BOOL CloseResult = CloseHandle(Handle);
#if PLATFORMFILETRACE_ENABLED
		if (CloseResult)
		{
			TRACE_PLATFORMFILE_END_CLOSE(SavedHandle);
		}
		else
		{
			TRACE_PLATFORMFILE_FAIL_CLOSE(SavedHandle);
		}
#else
		(void)CloseResult;
#endif
	}
	else
	{
		TRACE_PLATFORMFILE_FAIL_OPEN(Filename);
	}
}

FDateTime FWindowsPlatformFile::GetAccessTimeStamp(const TCHAR* Filename)
{
	WIN32_FILE_ATTRIBUTE_DATA Info;
	if (GetFileAttributesExW(*FNormalizedFilename(Filename), GetFileExInfoStandard, &Info))
	{
		return UE::WindowsPlatformFile::Private::WindowsFileTimeToUEDateTime(Info.ftLastAccessTime);
	}

	return FDateTime::MinValue();
}

FString FWindowsPlatformFile::GetFilenameOnDisk(const TCHAR* Filename)
{
	auto ConvertToUnrealPathInline = [](FString& Path)
	{
		// Remove the Windows device path prefix.
		if (Path.StartsWith(TEXT("\\\\?\\UNC\\"), ESearchCase::CaseSensitive))
		{
			Path.RemoveAt(2, 6); // remove ?\UNC\ to convert \\?\UNC\Path\... to \\Path\...
		}
		else
		{
			Path.RemoveFromStart(TEXT("\\\\?\\"), ESearchCase::CaseSensitive);
		}

		// Convert the result back into an UnrealPath (\\ -> /)
		Path.ReplaceCharInline(TEXT('\\'), TEXT('/'), ESearchCase::CaseSensitive);
	};

	FString Result = *FNormalizedFilename(Filename);
	FString NormalizedInputPath = Result;
	ConvertToUnrealPathInline(NormalizedInputPath);

	TRACE_PLATFORMFILE_BEGIN_OPEN(Filename);
	HANDLE hFile = CreateFile(*Result,
		FILE_READ_ATTRIBUTES, FILE_SHARE_READ | FILE_SHARE_WRITE | FILE_SHARE_DELETE,
		NULL, OPEN_EXISTING,
		FILE_FLAG_BACKUP_SEMANTICS, // Required for opening a directory, which Filename might be
		NULL);
	// If the file exists on disk, read the capitalization from the path on disk, otherwise just return the (normalized) input filename
	if (hFile == INVALID_HANDLE_VALUE)
	{
		TRACE_PLATFORMFILE_FAIL_OPEN(Filename);
		return NormalizedInputPath;
	}
	TRACE_PLATFORMFILE_END_OPEN(hFile);
	ON_SCOPE_EXIT
	{
		TRACE_PLATFORMFILE_BEGIN_CLOSE(hFile);
#if PLATFORMFILETRACE_ENABLED
		// MSVC static analysis has a rule that reports the argument to CloseHandle as uninitialized memory after the call to CloseHandle, so we have to save it ahead of time
		uint64 SavedHFile = uint64(hFile);
#endif
		BOOL CloseResult = CloseHandle(hFile);
#if PLATFORMFILETRACE_ENABLED
		if (CloseResult)
		{
			TRACE_PLATFORMFILE_END_CLOSE(SavedHFile);
		}
		else
		{
			TRACE_PLATFORMFILE_FAIL_CLOSE(SavedHFile);
		}
#else
		(void)CloseResult;
#endif
	};

	for (uint32 Length = Result.Len() + 10;;)
	{
		TArray<TCHAR, FString::AllocatorType>& CharArray = Result.GetCharArray();
		CharArray.SetNum(Length);

		Length = GetFinalPathNameByHandle(hFile, CharArray.GetData(), CharArray.Num(), FILE_NAME_NORMALIZED);
		if (Length == 0)
		{
			// Error - return normalized filename.
			return NormalizedInputPath;
		}
		if (Length < (uint32)CharArray.Num())
		{
			CharArray.SetNum(Length + 1);
			break;
		}
	}

	ConvertToUnrealPathInline(Result);

	// Check that the result only differs by case for the full path - otherwise do a slower approach which builds the path up by component.
	// This is to ensure symbolic links aren't followed.
	if (!Result.Equals(NormalizedInputPath, ESearchCase::IgnoreCase))
	{
		Result.Reset();

		FString AccumulatedFilename;
		AccumulatedFilename.Reserve(Result.Len());

		TArray<uint8> FileNameInfoTempBuffer;
		FileNameInfoTempBuffer.AddUninitialized(sizeof(FILE_NAME_INFO) + NormalizedInputPath.Len() * sizeof(TCHAR));
		FPathViews::IterateComponents(
			NormalizedInputPath,
			[&AccumulatedFilename, &FileNameInfoTempBuffer, &Result](FStringView Component)
			{
				AccumulatedFilename /= Component;

				if (Result.IsEmpty())
				{
					Result = Component;
					if (Component.EndsWith(':'))
					{
						Result.ToUpperInline();
					}
					return;
				}

				HANDLE hFile = CreateFile(
					*AccumulatedFilename,
					FILE_READ_ATTRIBUTES, FILE_SHARE_READ | FILE_SHARE_WRITE | FILE_SHARE_DELETE,
					NULL,
					OPEN_EXISTING,
					FILE_FLAG_OPEN_REPARSE_POINT | FILE_FLAG_BACKUP_SEMANTICS, // FILE_FLAG_OPEN_REPARSE_POINT prevents symlinks from being followed. FILE_FLAG_BACKUP_SEMANTICS is required for opening a directory, which AccumulatedFilename might be.
					NULL
				);
				if (hFile == INVALID_HANDLE_VALUE)
				{
					// File is probably missing - leave Component unchanged
					Result /= Component;
					return;
				}
				ON_SCOPE_EXIT
				{
					CloseHandle(hFile);
				};

				for (;;)
				{
					FILE_NAME_INFO* BufferPtr = (FILE_NAME_INFO*)FileNameInfoTempBuffer.GetData();
					BOOL bResult = GetFileInformationByHandleEx(hFile, FileNameInfo, BufferPtr, (DWORD)FileNameInfoTempBuffer.Num());
					if (bResult)
					{
						Result /= FPathViews::GetPathLeaf(MakeStringView(BufferPtr->FileName, BufferPtr->FileNameLength / sizeof(*BufferPtr->FileName)));
						return;
					}

					// If the error wasn't due to a too-small buffer - leave Component unchanged
					if (GetLastError() != ERROR_MORE_DATA)
					{
						Result /= Component;
						return;
					}

					// Increase buffer size and try again
					FileNameInfoTempBuffer.SetNumUninitialized(sizeof(FILE_NAME_INFO) + BufferPtr->FileNameLength);
				}
			}
		);

		// If it's still not a case variant, just return the normalized input.
		if (!Result.Equals(NormalizedInputPath, ESearchCase::IgnoreCase))
		{
			return NormalizedInputPath;
		}
	}

	return Result;
}

ESymlinkResult FWindowsPlatformFile::IsSymlink(const TCHAR* Filename)
{
	const uint32 Result = GetFileAttributesW(Filename);
	if (Result != 0xFFFFFFFF && (Result & FILE_ATTRIBUTE_REPARSE_POINT))
	{
		return ESymlinkResult::Symlink;
	}
	return ESymlinkResult::NonSymlink;
}

bool FWindowsPlatformFile::HasMarkOfTheWeb(FStringView Filename, FString* OutSourceURL)
{
	FNormalizedFilename StreamPath(Filename);
	StreamPath.Append(FileConstants::ZoneIdentifierStreamSuffix);

	const uint32 ZoneId = GetPrivateProfileIntW(TEXT("ZoneTransfer"), TEXT("ZoneId"), -1 /* URLZONE_INVALID */, *StreamPath);
	if (ZoneId != 2 /* URLZONE_TRUSTED */ && ZoneId != 3 /* URLZONE_INTERNET */ && ZoneId != 4 /* URLZONE_UNTRUSTED */)
	{
		return false;
	}

	if (OutSourceURL != nullptr)
	{
		// Can't use FConfigFile to parse ini because it treats double slashes in the url (e.g. HostUrl=http://...) as the beginning of a comment.
		TCHAR HostUrl[2048]; // @todo: remove hardcoded url length limit by using a simple standalone ini parser
		const int32 HostUrlLen = static_cast<int32>(GetPrivateProfileStringW(TEXT("ZoneTransfer"), TEXT("HostUrl"), nullptr, HostUrl, static_cast<DWORD>(GetNum(HostUrl)), *StreamPath));
		if (0 < HostUrlLen && HostUrlLen <= GetNum(HostUrl))
		{
			*OutSourceURL = FString::ConstructFromPtrSize(HostUrl, HostUrlLen);
		}
	}

	return true;
}

bool FWindowsPlatformFile::SetMarkOfTheWeb(FStringView Filename, bool bNewStatus, const FString* InSourceURL)
{
	FNormalizedFilename StreamPath(Filename);
	StreamPath.Append(FileConstants::ZoneIdentifierStreamSuffix);

	if (!bNewStatus)
	{
		return !!DeleteFileW(*StreamPath) || GetLastError() == ERROR_FILE_NOT_FOUND;
	}

	if (!WritePrivateProfileStringW(TEXT("ZoneTransfer"), TEXT("ZoneId"), TEXT("3") /* URLZONE_INTERNET */, *StreamPath))
	{
		return false;
	}

	if (InSourceURL != nullptr)
	{
		if (!WritePrivateProfileStringW(TEXT("ZoneTransfer"), TEXT("HostUrl"), **InSourceURL, *StreamPath))
		{
			return false;
		}
	}

	return true;
}

#if USE_WINDOWS_ASYNC_IMPL
IAsyncReadFileHandle* FWindowsPlatformFile::OpenAsyncRead(const TCHAR* Filename, bool bAllowWrite /*= false*/)
{
	uint32  Access = GENERIC_READ;
	uint32  WinFlags = FILE_SHARE_READ | (bAllowWrite ? FILE_SHARE_WRITE : 0) | GetFileShareDeleteFlagIfEnabled();
	uint32  Create = OPEN_EXISTING;


	FString NormalizedFilename = WindowsNormalizedFilename(Filename);
	TRACE_PLATFORMFILE_BEGIN_OPEN(Filename);
	HANDLE Handle = CreateFileW(*NormalizedFilename, Access, WinFlags, NULL, Create, FILE_ATTRIBUTE_NORMAL | FILE_FLAG_OVERLAPPED, NULL);
#if PLATFORMFILETRACE_ENABLED
	if (Handle != INVALID_HANDLE_VALUE)
	{
		TRACE_PLATFORMFILE_END_OPEN(Handle);
	}
	else
	{
		TRACE_PLATFORMFILE_FAIL_OPEN(Filename);
	}
#endif

	// we can't really fail here because this is intended to be an async open
	return new FMicrosoftAsyncReadFileHandle(Handle, *NormalizedFilename);

}
#endif

FFileOpenResult FWindowsPlatformFile::OpenRead(const TCHAR* Filename, EOpenReadFlags Flags)
{
	const uint32 Access				= GENERIC_READ;
	const uint32 ShareFlags			= FILE_SHARE_READ | GetFileShareDeleteFlagIfEnabled() |
										(EnumHasAnyFlags(Flags, EOpenReadFlags::AllowWrite) ? FILE_SHARE_WRITE : 0) |
										(EnumHasAnyFlags(Flags, EOpenReadFlags::AllowDelete) ? FILE_SHARE_DELETE : 0);
	const uint32 FlagsAndAttributes = (!EnumHasAnyFlags(Flags, EOpenReadFlags::NoBuffering) ? FILE_ATTRIBUTE_NORMAL :
										FILE_FLAG_NO_BUFFERING |	// Disables OS caching of file data
										FILE_ATTRIBUTE_TEMPORARY);	// Disables OS caching of file metadata
	const uint32 Create				= OPEN_EXISTING;

#define USE_OVERLAPPED_IO (!IS_PROGRAM && !WITH_EDITOR)		// Use straightforward synchronous I/O in cooker/editor

	TRACE_PLATFORMFILE_BEGIN_OPEN(Filename);
#if USE_OVERLAPPED_IO
	HANDLE Handle    = CreateFileW(*FNormalizedFilename(Filename), Access, ShareFlags, NULL, Create, FlagsAndAttributes | FILE_FLAG_OVERLAPPED, NULL);
	if (Handle != INVALID_HANDLE_VALUE)
	{
		TRACE_PLATFORMFILE_END_OPEN(Handle);
		FAsyncBufferedFileReaderWindows* FileHandle = new FAsyncBufferedFileReaderWindows(Handle, Access, ShareFlags, FlagsAndAttributes | FILE_FLAG_OVERLAPPED);
		return MakeValue(FileHandle);
	}
#else
	HANDLE Handle = CreateFileW(*FNormalizedFilename(Filename), Access, ShareFlags, NULL, Create, FlagsAndAttributes, NULL);
	if (Handle != INVALID_HANDLE_VALUE)
	{
		TRACE_PLATFORMFILE_END_OPEN(Handle);

		FFileHandleWindows* FileHandle = new FFileHandleWindows(Handle, Access, ShareFlags, FlagsAndAttributes);

		// Some operations can fail during the handle initialization, so we
		// double check that the handle is valid before returning it
		if (FileHandle->IsValid())
		{
			return MakeValue(FileHandle);
		}
		else
		{
			delete FileHandle;
			return MakeError(TEXT("Failed to create FFileHandleWindows"));
		}
	}
#endif //USE_OVERLAPPED_IO
	else
	{
		TRACE_PLATFORMFILE_FAIL_OPEN(Filename);
		return MakeError(TEXT("Failed to open file for reading"), FPlatformMisc::GetLastError());
	}
}

IFileHandle* FWindowsPlatformFile::OpenRead(const TCHAR* Filename, bool bAllowWrite)
{
	const EOpenReadFlags Flags = bAllowWrite ? EOpenReadFlags::AllowWrite : EOpenReadFlags::None;
	FFileOpenResult Result = OpenRead(Filename, Flags);
	if (Result.HasError())
	{
		return nullptr;
	}
	return Result.StealValue().Release();
}

FFileOpenResult FWindowsPlatformFile::OpenReadNoBuffering(const TCHAR* Filename, EOpenReadFlags Flags)
{
	const uint32 Access				= GENERIC_READ;
	const uint32 ShareFlags			= FILE_SHARE_READ | GetFileShareDeleteFlagIfEnabled() |
										(EnumHasAnyFlags(Flags, EOpenReadFlags::AllowWrite) ? FILE_SHARE_WRITE : 0) |
										(EnumHasAnyFlags(Flags, EOpenReadFlags::AllowDelete) ? FILE_SHARE_DELETE : 0);
	const uint32 FlagsAndAttributes = FILE_FLAG_OVERLAPPED | 
										(!EnumHasAnyFlags(Flags, EOpenReadFlags::NoBuffering) ? FILE_ATTRIBUTE_NORMAL :
										FILE_FLAG_NO_BUFFERING |	// Disables OS caching of file data
										FILE_ATTRIBUTE_TEMPORARY);	// Disables OS caching of file metadata
	const uint32 Create				= OPEN_EXISTING;

	TRACE_PLATFORMFILE_BEGIN_OPEN(Filename);
	HANDLE Handle = CreateFileW(*FNormalizedFilename(Filename), Access, ShareFlags, NULL, Create, FlagsAndAttributes, NULL);
	if (Handle != INVALID_HANDLE_VALUE)
	{
		TRACE_PLATFORMFILE_END_OPEN(Handle);

		FFileHandleWindows* FileHandle = new FFileHandleWindows(Handle, Access, ShareFlags, FlagsAndAttributes);

		// Some operations can fail during the handle initialization, so we
		// double check that the handle is valid before returning it
		if (FileHandle->IsValid())
		{
			return MakeValue(FileHandle);
		}
		else
		{
			delete FileHandle;
			return MakeError(TEXT("Failed to create FFileHandleWindows"));
		}
	}
	else
	{
		TRACE_PLATFORMFILE_FAIL_OPEN(Filename);
		return MakeError(TEXT("Failed to open file for reading"), FPlatformMisc::GetLastError());
	}
}

IFileHandle* FWindowsPlatformFile::OpenReadNoBuffering(const TCHAR* Filename, bool bAllowWrite)
{
	const EOpenReadFlags Flags = bAllowWrite ? EOpenReadFlags::AllowWrite : EOpenReadFlags::None;
	FFileOpenResult Result = OpenReadNoBuffering(Filename, Flags);
	if (Result.HasError())
	{
		return nullptr;
	}
	return Result.StealValue().Release();
}

IFileHandle* FWindowsPlatformFile::OpenWrite(const TCHAR* Filename, bool bAppend, bool bAllowRead)
{
	uint32  Access    = GENERIC_WRITE | (bAllowRead ? GENERIC_READ : 0);
	uint32  WinFlags  = bAllowRead ? FILE_SHARE_READ : 0;
	WinFlags |= GetFileShareDeleteFlagIfEnabled();

	uint32  Create    = bAppend ? OPEN_ALWAYS : CREATE_ALWAYS;
	TRACE_PLATFORMFILE_BEGIN_OPEN(Filename);
	HANDLE Handle    = CreateFileW(*FNormalizedFilename(Filename), Access, WinFlags, NULL, Create, FILE_ATTRIBUTE_NORMAL, NULL);
	if(Handle != INVALID_HANDLE_VALUE)
	{
		TRACE_PLATFORMFILE_END_OPEN(Handle);
		FFileHandleWindows* PlatformFileHandle = new FFileHandleWindows(Handle, Access, WinFlags, 0);

		// Some operations can fail during the handle initialization, so we
		// double check that the handle is valid before returning it
		if (PlatformFileHandle->IsValid())
		{
			if (bAppend)
			{
				PlatformFileHandle->SeekFromEnd(0);
			}
			return PlatformFileHandle;
		}
		else
		{
			delete PlatformFileHandle;

			return nullptr;
		}
	}
	else
	{
		TRACE_PLATFORMFILE_FAIL_OPEN(Filename);
		return nullptr;
	}
}

FOpenMappedResult FWindowsPlatformFile::OpenMappedEx(const TCHAR* Filename, EOpenReadFlags OpenOptions, int64 MaximumSize)
{
	EOpenMappedFlags ActualOpenOptions = EOpenMappedFlags::None;
	if (EnumHasAnyFlags(OpenOptions, EOpenReadFlags::AllowWrite))
	{
		ActualOpenOptions |= EOpenMappedFlags::OpenForWrite;
		ActualOpenOptions |= EOpenMappedFlags::AllowRead;
	}
	if (EnumHasAnyFlags(OpenOptions, EOpenReadFlags::AllowDelete))
	{
		ActualOpenOptions |= EOpenMappedFlags::AllowDelete;
	}

	return OpenMappedEx2(Filename, ActualOpenOptions, MaximumSize);
}

FOpenMappedResult FWindowsPlatformFile::OpenMappedEx2(const TCHAR* Filename, EOpenMappedFlags OpenOptions, int64 MaximumSize)
{
	uint32  Access = GENERIC_READ;
	uint32  WinFlags = FILE_SHARE_READ |
		(EnumHasAnyFlags(OpenOptions, EOpenMappedFlags::AllowWrite) ? FILE_SHARE_WRITE : 0) |
		(EnumHasAnyFlags(OpenOptions, EOpenMappedFlags::AllowDelete) ? FILE_SHARE_DELETE : 0) |
		GetFileShareDeleteFlagIfEnabled();
	uint32  Create = OPEN_EXISTING;
	DWORD	Protect = PAGE_READONLY;

	if (EnumHasAnyFlags(OpenOptions, EOpenMappedFlags::OpenForWrite))
	{
		Access = GENERIC_READ | GENERIC_WRITE;
		WinFlags = 
			(EnumHasAnyFlags(OpenOptions, EOpenMappedFlags::AllowRead) ? FILE_SHARE_READ : 0) |
			(EnumHasAnyFlags(OpenOptions, EOpenMappedFlags::AllowDelete) ? FILE_SHARE_DELETE : 0) |
			GetFileShareDeleteFlagIfEnabled();
		Create = EnumHasAnyFlags(OpenOptions, EOpenMappedFlags::OpenExisting) ? OPEN_EXISTING : CREATE_ALWAYS;
		Protect = PAGE_READWRITE;

		if (MaximumSize < 1)
		{
			if (EnumHasAnyFlags(OpenOptions, EOpenMappedFlags::OpenExisting))
			{
				const int64 Size = FileSize(Filename);
				if (Size < 1)
				{
					return MakeError(TEXT("File size is 0 or the file does not exist and no maximum size provided."));
				}

				MaximumSize = Size;
			}
			else
			{
				return MakeError(TEXT("No maximum size provided."));
			}
		}
	}
	else
	{
		const int64 Size = FileSize(Filename);
		if (Size < 1)
		{
			return MakeError(TEXT("File size is 0 or the file does not exist"));
		}

		if (MaximumSize < 1 || MaximumSize > Size)
		{
			MaximumSize = Size;
		}
	}

	FNormalizedFilename NormalizedFilename = FNormalizedFilename(Filename);

	TRACE_PLATFORMFILE_BEGIN_OPEN(Filename);
	HANDLE Handle = CreateFileW(*NormalizedFilename, Access, WinFlags, NULL, Create, FILE_ATTRIBUTE_NORMAL, NULL);
	if (Handle != INVALID_HANDLE_VALUE)
	{
		TRACE_PLATFORMFILE_END_OPEN(Handle);
	}
	else
	{
		TRACE_PLATFORMFILE_FAIL_OPEN(Filename);
		return MakeError(FString::Printf(TEXT("Failed to open file '%s'"), *NormalizedFilename));
	}

	HANDLE MappingHandle = CreateFileMapping(Handle, NULL, Protect, (DWORD)(MaximumSize >> 32), (DWORD)(MaximumSize), NULL);
	if (MappingHandle == NULL)
	{
		TRACE_PLATFORMFILE_BEGIN_CLOSE(Handle);
#if PLATFORMFILETRACE_ENABLED
		// MSVC static analysis has a rule that reports the argument to CloseHandle as uninitialized memory after the call to CloseHandle, so we have to save it ahead of time
		uint64 SavedHandle = uint64(Handle);
#endif
		BOOL CloseResult = CloseHandle(Handle);
#if PLATFORMFILETRACE_ENABLED
		if (CloseResult)
		{
			TRACE_PLATFORMFILE_END_CLOSE(SavedHandle);
		}
		else
		{
			TRACE_PLATFORMFILE_FAIL_CLOSE(SavedHandle);
		}
#else
		(void)CloseResult;
#endif
		return MakeError(FString::Printf(TEXT("Failed to create file mapping for a file'%s'"), *NormalizedFilename));
	}
	return MakeValue(new FMappedFileHandleWindows(Handle, MappingHandle, MaximumSize, Filename));
}

bool FWindowsPlatformFile::DirectoryExists(const TCHAR* Directory)
{
	// Empty Directory is the current directory so assume it always exists.
	bool bExists = !FCString::Strlen(Directory);
	if (!bExists) 
	{
		uint32 Result = GetFileAttributesW(*FNormalizedDirectory(Directory));
		bExists = (Result != 0xFFFFFFFF && (Result & FILE_ATTRIBUTE_DIRECTORY));
	}
	return bExists;
}

bool FWindowsPlatformFile::CreateDirectory(const TCHAR* Directory)
{
	return CreateDirectoryW(*FNormalizedDirectory(Directory), NULL) || GetLastError() == ERROR_ALREADY_EXISTS;
}

bool FWindowsPlatformFile::DeleteDirectory(const TCHAR* Directory)
{
	RemoveDirectoryW(*FNormalizedDirectory(Directory));
	uint32 LastError = GetLastError();
	const bool bSucceeded = !DirectoryExists(Directory);
	if (!bSucceeded)
	{
		SetLastError(LastError);
	}
	return bSucceeded;
}

FFileStatData FWindowsPlatformFile::GetStatData(const TCHAR* FilenameOrDirectory)
{
	WIN32_FILE_ATTRIBUTE_DATA Info;
	if (GetFileAttributesExW(*FNormalizedFilename(FilenameOrDirectory), GetFileExInfoStandard, &Info))
	{
		const bool bIsDirectory = !!(Info.dwFileAttributes & FILE_ATTRIBUTE_DIRECTORY);

		int64 FileSize = -1;
		if (!bIsDirectory)
		{
			LARGE_INTEGER li;
			li.HighPart = Info.nFileSizeHigh;
			li.LowPart = Info.nFileSizeLow;
			FileSize = static_cast<int64>(li.QuadPart);
		}

		return FFileStatData(
			UE::WindowsPlatformFile::Private::WindowsFileTimeToUEDateTime(Info.ftCreationTime),
			UE::WindowsPlatformFile::Private::WindowsFileTimeToUEDateTime(Info.ftLastAccessTime),
			UE::WindowsPlatformFile::Private::WindowsFileTimeToUEDateTime(Info.ftLastWriteTime),
			FileSize, 
			bIsDirectory,
			!!(Info.dwFileAttributes & FILE_ATTRIBUTE_READONLY)
			);
	}

	return FFileStatData();
}

bool FWindowsPlatformFile::IterateDirectory(const TCHAR* Directory, FDirectoryVisitor& Visitor)
{
	TStringBuilder<256> DirTemp;
	DirTemp << Directory;
	const int32 DirLen = DirTemp.Len();
	return IterateDirectoryCommon(Directory, [&](const WIN32_FIND_DATAW& InData) -> bool
	{
		DirTemp.RemoveSuffix(DirTemp.Len() - DirLen);
		FPathViews::Append(DirTemp, InData.cFileName);
		const bool bIsDirectory = !!(InData.dwFileAttributes & FILE_ATTRIBUTE_DIRECTORY);
		return Visitor.CallShouldVisitAndVisit(*DirTemp, bIsDirectory);
	});
}

bool FWindowsPlatformFile::IterateDirectoryStat(const TCHAR* Directory, FDirectoryStatVisitor& Visitor)
{
	const FString DirectoryStr = Directory;
	return IterateDirectoryCommon(Directory, [&](const WIN32_FIND_DATAW& InData) -> bool
	{
		const bool bIsDirectory = !!(InData.dwFileAttributes & FILE_ATTRIBUTE_DIRECTORY);

		int64 FileSize = -1;
		if (!bIsDirectory)
		{
			LARGE_INTEGER li;
			li.HighPart = InData.nFileSizeHigh;
			li.LowPart = InData.nFileSizeLow;
			FileSize = static_cast<int64>(li.QuadPart);
		}

		return Visitor.CallShouldVisitAndVisit(
			*(DirectoryStr / InData.cFileName), 
			FFileStatData(
				UE::WindowsPlatformFile::Private::WindowsFileTimeToUEDateTime(InData.ftCreationTime),
				UE::WindowsPlatformFile::Private::WindowsFileTimeToUEDateTime(InData.ftLastAccessTime),
				UE::WindowsPlatformFile::Private::WindowsFileTimeToUEDateTime(InData.ftLastWriteTime),
				FileSize, 
				bIsDirectory,
				!!(InData.dwFileAttributes & FILE_ATTRIBUTE_READONLY)
				)
			);
	});
}
	
// Forced not inline to reduce stack space usage since IterateDirectoryCommon might be recursive
FORCENOINLINE HANDLE FWindowsPlatformFile::FindFirstFileWithWildcard(const TCHAR* Directory, WIN32_FIND_DATAW& OutData)
{
	return FindFirstFileW(*(FNormalizedFilename(Directory, TEXTVIEW("*.*"))), &OutData);
}

bool FWindowsPlatformFile::IterateDirectoryCommon(const TCHAR* Directory, const TFunctionRef<bool(const WIN32_FIND_DATAW&)>& Visitor)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FWindowsPlatformFile_IterateDirectoryCommon_WithCallback);

	WIN32_FIND_DATAW Data;
	HANDLE Handle = FindFirstFileWithWildcard(Directory, /* Out */ Data);
	if (Handle != INVALID_HANDLE_VALUE)
	{
		bool bResult = true;
		do
		{
			if (FCString::Strcmp(Data.cFileName, TEXT(".")) && FCString::Strcmp(Data.cFileName, TEXT("..")))
			{
				bResult = Visitor(Data);
			}
		} while (bResult && FindNextFileW(Handle, &Data));
		FindClose(Handle);
		return bResult;
	}
	else
	{
		// FindFirstFileWithWildcard can return an invalid handle even for directories that exist on disk, due to flags
		// on the directory that instruct the operating system to hide them from various listings. e.g.
		//     FILE_ATTRIBUTE_HIDDEN: The file or directory is hidden. It is not included in an ordinary directory listing.
		//     FILE_ATTRIBUTE_REPARSE_POINT: A file or directory that has an associated reparse point, or a file that is a symbolic link.
		//     FILE_ATTRIBUTE_SYSTEM: A file or directory that the operating system uses a part of, or uses exclusively.
		// 
		// But unfortunately, FindNextFileW on the parent directory will still report those hidden directories, and we
		// therefore report them to our caller. IPlatformFile::IterateDirectoryRecursively on the parent
		// directory reasonably expects that for these directories we report, calling IterateDirectory on them does
		// not return false (unless the Visitor returns false).
		//
		// To workaround this issue, we return true for these hidden directories.
		// But we still need to return false if the directory does not exist.
		// DirectoryExists uses the function GetFileAttributes to decide whether the directory exists.
		return DirectoryExists(Directory);
	}
}

IPlatformFile& IPlatformFile::GetPlatformPhysical()
{
#if PLATFORM_USE_PLATFORM_FILE_MANAGED_STORAGE_WRAPPER
	static TManagedStoragePlatformFile<FWindowsPlatformFile> Singleton;
#else
	static FWindowsPlatformFile Singleton;	
#endif
	return Singleton;
}
