// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * The FileJournal API on Windows is implemented using Windows' USN Journal (aka NTFS Journal).
 *
 * USN stands for update sequence number.
 * The Journal has to be turned on as an fsutil feature on each drive. When it is enabled on a drive,
 * all writes to files on the drive cause the recording of an entry describing the modification into the journal.
 * The journal has a fixed size (specified at creation) and will drop older entries to make room for new ones.
 * 
 * We only enable it on editor because it has not been requested for in-game use and we want to reduce executable size.
 * 
 * Some features of the Journal require structs and enums defined in _WIN32_WINNT_WIN8 or higher.
 * If we are compiling against a lower version of the windows library, we define the structs and enums
 * we need. The API will still work (assuming the machine running our process supports it) because the library
 * just forwards our function call on to the dll on the machine. If the current machine is running Windows7
 * or lower, we gracefully fail the journal operations.
 *
 */

#include "Windows/WindowsPlatformFilePrivate.h"

#include "Containers/StringView.h"
#include "Logging/LogVerbosity.h"
#include "Misc/Paths.h"
#include "Misc/PathViews.h"

#define UE_WINDOWS_PLATFORM_FILEJOURNAL_ENABLED WITH_EDITOR 

#if UE_WINDOWS_PLATFORM_FILEJOURNAL_ENABLED
#include "Windows/WindowsPlatform.h" // Defines LogWindows, must come before CoreGlobals.h

#include "Containers/Map.h"
#include "Containers/Set.h"
#include "CoreGlobals.h"
#include "HAL/PlatformMisc.h"
#include "Logging/LogMacros.h"
#include "Misc/ScopeExit.h"
#include "Misc/StringBuilder.h"
#include "Windows/WindowsHWrapper.h"

// windows.h -> winscard.h -> winioctl.h is excluded by both UE_MINIMAL_WINDOWS_INCLUDE and NOCRYPT, which we define in
// WindowsHWrapper -> MinWindows.h before including windows.h. Include winioctl.h directly since it was excluded
#if defined(WINDOWS_H_WRAPPER_GUARD)
#error WINDOWS_H_WRAPPER_GUARD already defined
#endif
#define WINDOWS_H_WRAPPER_GUARD
#include "Windows/AllowWindowsPlatformTypes.h"
#include "Windows/PreWindowsApi.h"
#include <winioctl.h>
#include "Windows/PostWindowsApi.h"
#include "Windows/HideWindowsPlatformTypes.h"
#undef WINDOWS_H_WRAPPER_GUARD

#endif

FString FWindowsPlatformFile::FileJournalGetVolumeName(FStringView Path)
{
	FString FullPath;
	if (FPathViews::IsDriveSpecifierWithoutRoot(Path))
	{
		// Already a drive specifier, and ConvertRelativePathToFull does not handle a rootless drivespecifier
		return FString(Path);
	}
	else
	{
		FullPath = FPaths::ConvertRelativePathToFull(FString(Path));
		FStringView VolumeName;
		FStringView Remainder;
		FPathViews::SplitVolumeSpecifier(FullPath, VolumeName, Remainder);
		if (VolumeName.EndsWith(TEXT(":")))
		{
			return FString(VolumeName);
		}
		return FString(); // Volume names of the form \\volumename are not supported by USNJournal
	}
}

#if UE_WINDOWS_PLATFORM_FILEJOURNAL_ENABLED 

#if _WIN32_WINNT < _WIN32_WINNT_WIN8
typedef struct _FILE_ID_EXTD_DIR_INFO {
	ULONG NextEntryOffset;
	ULONG FileIndex;
	LARGE_INTEGER CreationTime;
	LARGE_INTEGER LastAccessTime;
	LARGE_INTEGER LastWriteTime;
	LARGE_INTEGER ChangeTime;
	LARGE_INTEGER EndOfFile;
	LARGE_INTEGER AllocationSize;
	ULONG FileAttributes;
	ULONG FileNameLength;
	ULONG EaSize;
	ULONG ReparsePointTag;
	FILE_ID_128 FileId;
	WCHAR FileName[1];
} FILE_ID_EXTD_DIR_INFO, * PFILE_ID_EXTD_DIR_INFO;

constexpr _FILE_INFO_BY_HANDLE_CLASS FileIdExtdDirectoryInfo = (_FILE_INFO_BY_HANDLE_CLASS)(((int32)FileFullDirectoryRestartInfo) + 4);
constexpr _FILE_INFO_BY_HANDLE_CLASS FileIdExtdDirectoryRestartInfo = (_FILE_INFO_BY_HANDLE_CLASS)(((int32)FileIdExtdDirectoryInfo) + 1);

#endif

namespace UE::WindowsPlatformFileJournal::Private
{

bool FileJournalIsAvailableSystemWide();
bool FileJournalIsAvailableSystemWide(ELogVerbosity::Type* OutErrorLevel, FString* OutError);
EFileJournalResult CreateJournalWindowsHandle(const TCHAR* VolumeOrPath, HANDLE& OutHandle,
	FStringBuilderBase& OutVolumeName, FString* OutError);
EFileJournalResult CreateJournalWindowsHandle(const TCHAR* VolumeName, HANDLE& OutHandle, FString* OutError);
EFileJournalResult GetJournalDescriptor(HANDLE JournalWindowsHandle, const TCHAR* VolumeName,
	USN_JOURNAL_DATA_V1& OutJournalDescriptor, FString* OutError);
FString JournalGetLastErrorString(uint32 LastError);

FStringView GetFileName(FILE_ID_EXTD_DIR_INFO* Info);
FStringView GetFileName(USN_RECORD_V3* Record);

FFileJournalFileHandle ToFileHandle(uint64 ReferenceNumber);
FFileJournalFileHandle ToFileHandle(FILE_ID_128& ReferenceNumber);

uint8 GJournalAvailable = 0;

};

bool FWindowsPlatformFile::FileJournalIsAvailable(const TCHAR* VolumeOrPath, ELogVerbosity::Type* OutErrorLevel,
	FString* OutError)
{
	using namespace UE::WindowsPlatformFileJournal::Private;

	if (!FileJournalIsAvailableSystemWide(OutErrorLevel, OutError))
	{
		return false;
	}
	if (!VolumeOrPath)
	{
		return true;
	}

	FFileJournalId JournalId;
	FFileJournalEntryHandle Handle;
	EFileJournalResult Result = FileJournalGetLatestEntry(VolumeOrPath, JournalId, Handle, OutError);
	if (Result == EFileJournalResult::Success)
	{
		if (OutErrorLevel)
		{
			*OutErrorLevel = ELogVerbosity::Verbose;
		}
		return true;
	}
	else
	{
		if (OutErrorLevel)
		{
			*OutErrorLevel = ELogVerbosity::Error;
		}
		return false;
	}
}

uint64 FWindowsPlatformFile::FileJournalGetMaximumSize(const TCHAR* VolumeOrPath, ELogVerbosity::Type* OutErrorLevel,
	FString* OutError) const
{
	using namespace UE::WindowsPlatformFileJournal::Private;

	if (!VolumeOrPath)
	{
		return 0;
	}

	HANDLE JournalHandle;
	TStringBuilder<16> NormalizedVolumeName;
	EFileJournalResult Result = CreateJournalWindowsHandle(VolumeOrPath, JournalHandle, NormalizedVolumeName, OutError);
	if (Result != EFileJournalResult::Success)
	{
		if (OutErrorLevel)
		{
			*OutErrorLevel = ELogVerbosity::Error;
		}
		return 0;
	}
	ON_SCOPE_EXIT{ ::CloseHandle(JournalHandle); };

	USN_JOURNAL_DATA_V1 JournalDescriptor;
	Result = GetJournalDescriptor(JournalHandle, *NormalizedVolumeName, JournalDescriptor, OutError);
	if (Result != EFileJournalResult::Success)
	{
		if (OutErrorLevel)
		{
			*OutErrorLevel = ELogVerbosity::Error;
		}
		return 0;
	}
	
	return JournalDescriptor.MaximumSize;
}

EFileJournalResult FWindowsPlatformFile::FileJournalGetLatestEntry(const TCHAR* VolumeName,
	FFileJournalId& OutJournalId, FFileJournalEntryHandle& OutEntryHandle, FString* OutError)
{
	using namespace UE::WindowsPlatformFileJournal::Private;

	OutJournalId = FileJournalIdInvalid;
	OutEntryHandle = FileJournalEntryHandleInvalid;

	HANDLE JournalHandle;
	TStringBuilder<16> NormalizedVolumeName;
	EFileJournalResult Result = CreateJournalWindowsHandle(VolumeName, JournalHandle, NormalizedVolumeName,
		OutError);
	if (Result != EFileJournalResult::Success)
	{
		return Result;
	}
	ON_SCOPE_EXIT{ ::CloseHandle(JournalHandle); };

	USN_JOURNAL_DATA_V1 JournalDescriptor;
	Result = GetJournalDescriptor(JournalHandle, *NormalizedVolumeName, JournalDescriptor, OutError);
	if (Result != EFileJournalResult::Success)
	{
		return Result;
	}
	static_assert(sizeof(uint64) >= sizeof(JournalDescriptor.UsnJournalID), "We are storing JournalIDs as uint64");
	static_assert(sizeof(uint64) >= sizeof(USN), "We are storing USNs as uint64");
	OutJournalId = static_cast<FFileJournalId>(JournalDescriptor.UsnJournalID);
	OutEntryHandle = static_cast<FFileJournalEntryHandle>(JournalDescriptor.NextUsn);
	return EFileJournalResult::Success;
}

bool FWindowsPlatformFile::FileJournalIterateDirectory(const TCHAR* Directory,
	FDirectoryJournalVisitorFunc Visitor, FString* OutError)
{
	using namespace UE::WindowsPlatformFileJournal::Private;
	using namespace UE::WindowsPlatformFile::Private;

	if (!FileJournalIsAvailableSystemWide())
	{
		return IPlatformFile::FileJournalIterateDirectory(Directory, Visitor, OutError);
	}

	FNormalizedDirectory Normalized(Directory);
	HANDLE Handle = CreateFileW(*Normalized,
		GENERIC_READ,
		FILE_SHARE_READ,
		nullptr,
		OPEN_EXISTING,
		FILE_FLAG_BACKUP_SEMANTICS, // Required when opening a directory
		nullptr);
	if (Handle == INVALID_HANDLE_VALUE)
	{
		if (OutError)
		{
			uint32 LastError = GetLastError();
			*OutError = FString::Printf(TEXT("FileJournal: CreateFile('%s') failed; FileJournal feature is unavailable. LastError=%u (0x%08x) (%s)."),
				*Normalized, LastError, LastError, *JournalGetLastErrorString(LastError));
		}

		return false;
	}
	ON_SCOPE_EXIT{ ::CloseHandle(Handle); };

	// Add overflow to BufferSize to allow check for a null terminator by reading past the end of FileNameLength
	constexpr int BufferSize = 4096;
	constexpr int BufferSizeOverflow = 1;
	uint8 Buffer[BufferSize + BufferSizeOverflow];

	BOOL Result = GetFileInformationByHandleEx(Handle, ::FileIdExtdDirectoryRestartInfo, Buffer, BufferSize);
	if (!Result)
	{
		if (OutError)
		{
			uint32 LastError = FPlatformMisc::GetLastError();
			*OutError = FString::Printf(TEXT("Unexpected failure of GetFileInformationByHandleEx('%s') with error code %u (0x%08x) (%s)."),
				*Normalized, LastError, LastError, *JournalGetLastErrorString(LastError));
		}

		// TODO: Need to handle filenames with size > buffersize by checking GetLastError and passing in a dynamically sized buffer.
		return false;
	}

	TStringBuilder<1024> LocalAbsPath;
	LocalAbsPath << Directory;
	int32 SavedLocalAbsPathLen = LocalAbsPath.Len();

	for (;;) // For each call to GetFileInformationByHandleEx
	{
		DWORD CurrentOffset = 0;
		for (;;) // For every FILE_ID_EXTD_DIR_INFO in the buffer returned by GetFileInformationByHandleEx
		{
			FILE_ID_EXTD_DIR_INFO* Info = reinterpret_cast<FILE_ID_EXTD_DIR_INFO*>(Buffer + CurrentOffset);
			FStringView RelPath = GetFileName(Info);

			// GetFileInformationByHandleEx returns entries for . and .., but our contract is that we only
			// return entries for child paths
			if (RelPath != TEXTVIEW(".") && RelPath != TEXTVIEW(".."))
			{
				check(!RelPath.Contains(TEXT("/")) && !RelPath.Contains(TEXT("\\"))); // RemoveSuffix would break if we called AppendPath with an absolute path
				LocalAbsPath.RemoveSuffix(LocalAbsPath.Len() - SavedLocalAbsPathLen);
				FPathViews::AppendPath(LocalAbsPath, RelPath);

				FFileJournalData FileData;
				FILETIME LastWriteTime;
				LastWriteTime.dwLowDateTime = Info->LastWriteTime.LowPart;
				LastWriteTime.dwHighDateTime = Info->LastWriteTime.HighPart;
				FileData.ModificationTime = WindowsFileTimeToUEDateTime(LastWriteTime);
				FileData.bIsValid = true;
				FileData.bIsDirectory = !!(Info->FileAttributes & FILE_ATTRIBUTE_DIRECTORY);
				FileData.bIsReparsePoint = !!(Info->FileAttributes & FILE_ATTRIBUTE_REPARSE_POINT);
				FileData.bIsReadOnly = !!(Info->FileAttributes & FILE_ATTRIBUTE_READONLY);
				FileData.JournalHandle = ToFileHandle(Info->FileId);

				if (!Visitor(*LocalAbsPath, FileData))
				{
					return false;
				}
			}

			if (Info->NextEntryOffset == 0)
			{
				break;
			}
			CurrentOffset += Info->NextEntryOffset;
		}

		BOOL NextResult = GetFileInformationByHandleEx(Handle, ::FileIdExtdDirectoryInfo, Buffer, BufferSize);
		if (!NextResult)
		{
			// TODO: Need to handle filenames with size > buffersize by checking GetLastError and passing in a dynamically sized buffer.
			return true;
		}
	}
}

FFileJournalData FWindowsPlatformFile::FileJournalGetFileData(const TCHAR* FilenameOrDirectory, FString* OutError)
{
	using namespace UE::WindowsPlatformFileJournal::Private;
	using namespace UE::WindowsPlatformFile::Private;

	if (!FileJournalIsAvailableSystemWide())
	{
		return IPlatformFile::FileJournalGetFileData(FilenameOrDirectory);
	}

	FFileJournalData OutResult;
	FNormalizedFilename Normalized(FilenameOrDirectory);
	// We don't know whether FilenameOrDirectory is a directory, so we can't use FNormalizedDirectory and have to use
	// FNormalizedFile. Unlike FNormalizedDirectory, FNormalizedFile does not remove redundant terminating separators.
	// But CreateFileW requires that they be removed, so remove them.
	while (FPathViews::HasRedundantTerminatingSeparator(Normalized.ToView()))
	{
		Normalized.RemoveSuffix(1);
	}

	HANDLE Handle = CreateFileW(*Normalized,
		GENERIC_READ,
		FILE_SHARE_READ,
		nullptr,
		OPEN_EXISTING,
		FILE_FLAG_BACKUP_SEMANTICS, // Required when opening a directory
		nullptr);
	if (Handle == INVALID_HANDLE_VALUE)
	{
		if (OutError)
		{
			uint32 LastError = GetLastError();
			*OutError = FString::Printf(TEXT("FileJournal: CreateFile('%s') failed; FileJournal feature is unavailable. LastError=%u (0x%08x) (%s)."),
				*Normalized, LastError, LastError, *JournalGetLastErrorString(LastError));
		}
		return OutResult;
	}
	ON_SCOPE_EXIT{ ::CloseHandle(Handle); };

	// GetFileInformationByHandleEx returns information for as many paths in the directory as fit in the buffer
	// we provide. The first one is the directory itself, with filename '.'.
	// That's the only one we want to fetch for this call; we don't want to waste time copying data into our buffer
	// about the other files. Since the filename is a single character, and sizeof(FILE_ID_EXTD_DIR_INFO) includes the
	// first character of the filename, then the record for this filename will fit into a buffer of exactly
	// sizeof(FILE_ID_EXTD_DIR_INFO). But make the buffer bigger than that (but smaller than 2*FILE_ID_EXTD_DIR_INFO)
	// just to be on the safe side.
	static_assert(sizeof(FILE_ID_EXTD_DIR_INFO) > 20); // Our extra space should be smaller than half of a buffer
	uint8 Buffer[sizeof(FILE_ID_EXTD_DIR_INFO) + 10];

	BOOL Result = GetFileInformationByHandleEx(Handle, ::FileIdExtdDirectoryRestartInfo, Buffer, sizeof(Buffer));
	if (!Result)
	{
		uint32 LastError = FPlatformMisc::GetLastError();
		// ERROR_BAD_LENGTH means our buffer is too short, which indicates our comment above is wrong and we
		// need to fix the code. Other failures can happen and are not necessarily errors.
		if (LastError == ERROR_BAD_LENGTH)
		{
			UE_LOGF(LogWindows, Error, "Unexpected failure of GetFileInformationByHandleEx('%ls') with error code %u (0x%08x) (%ls).",
				*Normalized, LastError, LastError, *JournalGetLastErrorString(LastError));
		}

		if (OutError)
		{
			*OutError = FString::Printf(TEXT("Unexpected failure of GetFileInformationByHandleEx('%s') with error code %u (0x%08x) (%s)."),
				*Normalized, LastError, LastError, *JournalGetLastErrorString(LastError));
		}

		return OutResult;
	}

	FILE_ID_EXTD_DIR_INFO* Info = reinterpret_cast<FILE_ID_EXTD_DIR_INFO*>(Buffer);
	FStringView RelPath = GetFileName(Info);
	if (RelPath != TEXTVIEW("."))
	{
		UE_LOGF(LogWindows, Error, "Unexpected failure of GetFileInformationByHandleEx('%ls'). Expected it to return results for filename '.', but it returned filename %.*ls.",
			*Normalized, RelPath.Len(), RelPath.GetData());
		return OutResult;
	}

	OutResult.bIsDirectory = !!(Info->FileAttributes & FILE_ATTRIBUTE_DIRECTORY);
	OutResult.bIsReparsePoint = !!(Info->FileAttributes & FILE_ATTRIBUTE_REPARSE_POINT);
	OutResult.bIsReadOnly = !!(Info->FileAttributes & FILE_ATTRIBUTE_READONLY);
	FILETIME LastWriteTime;
	LastWriteTime.dwLowDateTime = Info->LastWriteTime.LowPart;
	LastWriteTime.dwHighDateTime = Info->LastWriteTime.HighPart;
	OutResult.ModificationTime = WindowsFileTimeToUEDateTime(LastWriteTime);
	OutResult.bIsValid = true;
	OutResult.JournalHandle = ToFileHandle(Info->FileId);

	return OutResult;
}

EFileJournalResult FWindowsPlatformFile::FileJournalReadModified(const TCHAR* VolumeName,
	const FFileJournalId& JournalIdOfStartingEntry, const FFileJournalEntryHandle& StartingJournalEntry,
	TMap<FFileJournalFileHandle, FString>& KnownDirectories, TSet<FString>& OutModifiedDirectories,
	FFileJournalEntryHandle& OutNextJournalEntry, FString* OutError)
{
	using namespace UE::WindowsPlatformFileJournal::Private;

	if (!FileJournalIsAvailableSystemWide(nullptr, OutError))
	{
		return IPlatformFile::FileJournalReadModified(VolumeName, JournalIdOfStartingEntry, StartingJournalEntry,
			KnownDirectories, OutModifiedDirectories, OutNextJournalEntry, nullptr);
	}

	EFileJournalResult Result;
	OutNextJournalEntry = StartingJournalEntry;

	// Records are expected to be aligned to a 64-bit boundary. The USN taking the first 64-bits.
	alignas(sizeof(uint64)) uint8 ReadBuffer[4096];

	HANDLE JournalWindowsHandle = NULL;
	TStringBuilder<16> NormalizedVolumeName;
	Result = CreateJournalWindowsHandle(VolumeName, JournalWindowsHandle, NormalizedVolumeName, OutError);
	if (Result != EFileJournalResult::Success)
	{
		return Result;
	}
	ON_SCOPE_EXIT{ ::CloseHandle(JournalWindowsHandle); };
	VolumeName = *NormalizedVolumeName;

	USN_JOURNAL_DATA_V1 JournalDescriptor;
	Result = GetJournalDescriptor(JournalWindowsHandle, VolumeName, JournalDescriptor,
		OutError);
	if (Result != EFileJournalResult::Success)
	{
		return Result;
	}

	if (static_cast<DWORDLONG>(JournalIdOfStartingEntry) != JournalDescriptor.UsnJournalID)
	{
		if (OutError)
		{
			*OutError = FString::Printf(
				TEXT("The JournalId StartingJournalEntry on volume '%s' came from a different journal. This can happen when the Journal has been destroyed and recreated. ")
				TEXT("StoredJournalId = %" UINT64_FMT ", CurrentJournalId = %" UINT64_FMT "."),
				VolumeName, static_cast<uint64>(JournalIdOfStartingEntry), static_cast<uint64>(JournalDescriptor.UsnJournalID));
		}
		return EFileJournalResult::JournalWrapped;
	}

	// FSCTL_READ_USN_JOURNAL does not return an error if the StartUsn is greater than the end of the current journal;
	// it returns no results and reports the input StartUsn as the next USN to read. We want this to be an error
	// condition similar to wrapping the journal. StartUSn == NextUSN is valid, but StartUSN > NextUSN is not.
	// Return a wrap code for that case.
	const USN LastJournalUSN = JournalDescriptor.NextUsn;
	USN CurrentJournalUSN = static_cast<USN>(StartingJournalEntry);
	if (CurrentJournalUSN > LastJournalUSN)
	{
		if (OutError)
		{
			*OutError = FString::Printf(
				TEXT("The stored value for StartingJournalEntry on volume '%s' is invalid; it is past the end of the current Journal. This can happen when the Journal has been destroyed and recreated. ")
				TEXT("StoredValue = %" UINT64_FMT ", CurrentEnd = %" UINT64_FMT "."),
				VolumeName, static_cast<uint64>(CurrentJournalUSN), static_cast<uint64>(JournalDescriptor.NextUsn));
		}
		return EFileJournalResult::JournalWrapped;
	}

	READ_USN_JOURNAL_DATA_V1 ReadJournalRequest;
	// The USN at which to begin reading the change journal, or 0 to start from beginning
	// ReadJournalRequest.StartUsn; // Set below in the loop
	// Filter for types of changes that should be returned, e.g. USN_REASON_DATA_EXTEND: The file or directory is added to
	ReadJournalRequest.ReasonMask = ~0;
	// Changes to the filesystem can be reported when made, or when all processes drop their handle to the file.
	// ReturnOnlyOnClose = 0 specifies as soon as made, non-zero specifies wait for handles to drop.
	ReadJournalRequest.ReturnOnlyOnClose = 0;
	// Timeout to use, in seconds, if BytesToWaitFor requests more data than exists in the change journal.
	ReadJournalRequest.Timeout = 0;
	// If FSCTL_READ_USN_JOURNAL operation requests more bytes than exist, how many bytes we should wait for before returning
	ReadJournalRequest.BytesToWaitFor = 0;
	// Journal to use, id is from JournalDescriptor returned by FSCTL_QUERY_USN_JOURNAL
	ReadJournalRequest.UsnJournalID = JournalDescriptor.UsnJournalID;
	// Minimum major version supported by our application, can be read from JournalDescriptor returned by FSCTL_QUERY_USN_JOURNAL
	ReadJournalRequest.MinMajorVersion = JournalDescriptor.MinSupportedMajorVersion;
	// Maximum major version supported by our application, can be read from JournalDescriptor returned by FSCTL_QUERY_USN_JOURNAL
	ReadJournalRequest.MaxMajorVersion = JournalDescriptor.MaxSupportedMajorVersion;

	DWORD ReadBufferResultsSize;
	uint8* RecordBytesEnd = ReadBuffer;

	// bTryShowFilenames is a debug feature, and is only available if process is privileged. The extra data it returns
	// is logged but otherwise unused. If set to true and the process is not privileged, this function will run more
	// slowly but will not fail.
	const bool bTryShowFilenames = false;
	bool bShowFilenames = bTryShowFilenames;
	bool bTriedAndFailedShowFilenames = false;
	bool bFirstCall = true;
	do
	{
		// FSCTL_READ_UNPRIVILEGED_USN_JOURNAL can read without elevation but does not show filenames.
		// FCSTL_READ_USN_JOURNAL requires elevation and shows filenames
		DWORD dwIoControlCode = bShowFilenames ? FSCTL_READ_USN_JOURNAL : FSCTL_READ_UNPRIVILEGED_USN_JOURNAL;
		bool bWasFirstCall = bFirstCall;
		bFirstCall = false;
		ReadJournalRequest.StartUsn = CurrentJournalUSN;

		BOOL ReadJournalResult = ::DeviceIoControl(
			JournalWindowsHandle,
			dwIoControlCode,
			&ReadJournalRequest,
			sizeof(ReadJournalRequest),
			ReadBuffer,
			sizeof(ReadBuffer),
			&ReadBufferResultsSize,
			NULL /* Overlapped */);
		if (!ReadJournalResult)
		{
			uint32 LastError = FPlatformMisc::GetLastError();
			if (bWasFirstCall && dwIoControlCode == FSCTL_READ_USN_JOURNAL && LastError == ERROR_ACCESS_DENIED)
			{
				bTriedAndFailedShowFilenames = true;
				bShowFilenames = false;
				continue; // Try again with the unprivileged version
			}
			else
			{
				switch (LastError)
				{
				case ERROR_JOURNAL_ENTRY_DELETED:
					if (OutError)
					{
						*OutError = FString::Printf(TEXT("NTFS Journal has wrapped for volume '%s'; the number of files written to this volume exceeded the buffer size of the journal and some entries were dropped.")
							TEXT("\r\nIf this happens frequently increase the buffer size of the journal:")
							TEXT("\r\nLaunch cmd.exe as admin and run command `fsutil usn createJournal %s m=<SizeInBytes>`")
							TEXT("\r\n<SizeInBytes> defaults to 33554432 (32MB), Recommended <SizeInBytes> is 0x40000000 (1GB). You can see the current value via `fsutil usn queryJournal %s`, 'Maximum Size' field."),
							VolumeName, VolumeName, VolumeName);
					}
					return EFileJournalResult::JournalWrapped;
				default:
					if (OutError)
					{
						*OutError = FString::Printf(
							TEXT("FSCTL_READ_UNPRIVILEGED_USN_JOURNAL failed for volume '%s'. GetLastError == %u (0x%08x) (%s)."),
							VolumeName, LastError, LastError, *JournalGetLastErrorString(LastError));
					}
					return EFileJournalResult::FailedReadJournal;
				}
			}
		}
		if (ReadBufferResultsSize <= sizeof(USN))
		{
			// There are no more USN entries or Journal Records left
			// Note, we check for <=. While theoretically we can read a USN, if there are no bytes for 
			// the records we must be out of data to read, so return without error.
			return EFileJournalResult::Success;
		}
		if (bTriedAndFailedShowFilenames)
		{
			UE_LOGF(LogWindows, Error, "FileJournalReadModified cannot show filenames because the current process is not elevated.");
			bTriedAndFailedShowFilenames = false;
		}
		uint8* RecordBytes = ReadBuffer;
		RecordBytesEnd = ReadBuffer + ReadBufferResultsSize;
		USN NextJournalUSN = *reinterpret_cast<USN*>(ReadBuffer);
		RecordBytes += sizeof(USN);
		while (RecordBytes < RecordBytesEnd)
		{
			USN_RECORD_V3* Record = reinterpret_cast<USN_RECORD_V3*>(RecordBytes);
			if (Record->MajorVersion != 3)
			{
				// If we need to support earlier version 2, we will need to intepret the data as USN_RECORD_V2
				if (OutError)
				{
					*OutError = FString::Printf(
						TEXT("FSCTL_READ_UNPRIVILEGED_USN_JOURNAL for volume '%s' returned a record of version %d.%d, but we only handle version 3.*."),
						VolumeName, (int)Record->MajorVersion, (int)Record->MinorVersion);
				}
				return EFileJournalResult::UnhandledJournalVersion;
			}

			FFileJournalFileHandle TargetJournalHandle = ToFileHandle(Record->FileReferenceNumber);
			FString* TargetPathName = KnownDirectories.Find(TargetJournalHandle);
			if (TargetPathName)
			{
				// If the TargetPath is one of our tracked directories itself, mark it as modified
				// because we might need to update its FileReferenceNumber.
				OutModifiedDirectories.Add(*TargetPathName);

				if (bShowFilenames)
				{
					FStringView Filename = GetFileName(Record);
					UE_LOGF(LogWindows, Display, "FileJournalReadModified: TargetDirectory %ls",
						*TargetJournalHandle.ToString());
				}
			}
			FFileJournalFileHandle ParentJournalHandle = ToFileHandle(Record->ParentFileReferenceNumber);
			FString* ParentDirectoryName = KnownDirectories.Find(ParentJournalHandle);
			if (ParentDirectoryName)
			{
				// If a file or directory that is a child under one of our tracked directories has changed,
				// then mark the tracked directory as modified because we need to rescan it to refresh its
				// list of existing child paths.
				OutModifiedDirectories.Add(*ParentDirectoryName);

				if (bShowFilenames)
				{
					FStringView Filename = GetFileName(Record);
					UE_LOGF(LogWindows, Display, "FileJournalReadModified: ParentDirectory %ls with file %.*ls",
						*ParentJournalHandle.ToString(), Filename.Len(), Filename.GetData());
				}
			}

			RecordBytes += Record->RecordLength;
		}

		// Prevent a possible journal bug from causing an infinite loop
		if (NextJournalUSN <= CurrentJournalUSN)
		{
			if (OutError)
			{
				*OutError = FString::Printf(
					TEXT("FSCTL_READ_UNPRIVILEGED_USN_JOURNAL returned an unexpected value; ReadBufferResultsSize == %u, but NextJournalUSN did not increase.")
					TEXT("Previous value = %" UINT64_FMT ". New value = %" UINT64_FMT "."),
					(uint32)ReadBufferResultsSize, static_cast<uint64>(CurrentJournalUSN), static_cast<uint64>(NextJournalUSN));
			}
			return EFileJournalResult::JournalInternalError;
		}

		static_assert(sizeof(uint64) >= sizeof(USN), "We are storing USNs as uint64");
		OutNextJournalEntry = static_cast<uint64>(NextJournalUSN);
		CurrentJournalUSN = NextJournalUSN;
	} while (CurrentJournalUSN < LastJournalUSN);
	return EFileJournalResult::Success;
}

namespace UE::WindowsPlatformFileJournal::Private
{

bool FileJournalIsAvailableSystemWide()
{
	switch (GJournalAvailable)
	{
	case 1:
		return true;
	case 2:
		return false;
	default:
		break;
	}
	return FileJournalIsAvailableSystemWide(nullptr, nullptr);
}

bool FileJournalIsAvailableSystemWide(ELogVerbosity::Type* OutErrorLevel, FString* OutError)
{
	switch (GJournalAvailable)
	{
	case 1:
		if (OutErrorLevel)
		{
			*OutErrorLevel = ELogVerbosity::Verbose;
		}
		if (OutError)
		{
			OutError->Empty();
		}
		return true;
	case 2:
		if (!OutErrorLevel && !OutError)
		{
			return false;
		}
		// Fall through and recompute so we can provide the information about why it failed
		break;
	default:
		break;
	}

	auto ReturnResult = [OutErrorLevel, OutError](bool bResult, ELogVerbosity::Type ErrorLevel, FString&& Error)
		{
			if (OutErrorLevel)
			{
				*OutErrorLevel = ErrorLevel;
			}
			if (OutError)
			{
				*OutError = MoveTemp(Error);
			}
			GJournalAvailable = bResult ? 0x1 : 0x2;
			return bResult;
		};

	FWindowsPlatformFile::FNormalizedDirectory Normalized(*FPaths::LaunchDir());
	HANDLE Handle = CreateFileW(*Normalized,
		GENERIC_READ,
		FILE_SHARE_READ,
		nullptr,
		OPEN_EXISTING,
		FILE_FLAG_BACKUP_SEMANTICS, // Required when opening a directory
		nullptr);
	if (Handle == INVALID_HANDLE_VALUE)
	{
		// Unexpected failure
		uint32 LastError = GetLastError();
		return ReturnResult(false, ELogVerbosity::Error, FString::Printf(
			TEXT("FileJournal: CreateFile('%s') failed; FileJournal feature is unavailable. LastError=%u (0x%08x) (%s)."),
			*Normalized, LastError, LastError, *JournalGetLastErrorString(LastError)));
	}
	ON_SCOPE_EXIT{ ::CloseHandle(Handle); };

	static_assert(sizeof(FILE_ID_EXTD_DIR_INFO) > 20); // Our extra space should be smaller than half of a buffer
	uint8 Buffer[sizeof(FILE_ID_EXTD_DIR_INFO) + 10];
	BOOL Result = GetFileInformationByHandleEx(Handle, ::FileIdExtdDirectoryRestartInfo, Buffer, sizeof(Buffer));
	if (Result)
	{
		// Available
		return ReturnResult(true, ELogVerbosity::Verbose, FString());
	}

	uint32 LastError = GetLastError();
	switch (LastError)
	{
	case ERROR_INVALID_PARAMETER:
		// Failed, but expected if the windows version is too low. FileIdExtdDirectoryRestartInfo is not recognized before windows 8.
		return ReturnResult(false, ELogVerbosity::Display, FString::Printf(
			TEXT("FileJournal requires Windows 8 or later; it is unavailable on the current platform. LastError=%u (0x%08x) (%s)."),
			LastError, LastError, *JournalGetLastErrorString(LastError)));
	default:
		// Unexpected failure
		return ReturnResult(false, ELogVerbosity::Error, FString::Printf(
			TEXT("FileJournal: FileIdExtdDirectoryRestartInfo('%s') failed; FileJournal feature is unavailable. LastError=%u (0x%08x) (%s)."),
			*Normalized, LastError, LastError, *JournalGetLastErrorString(LastError)));
	}
}

EFileJournalResult CreateJournalWindowsHandle(const TCHAR* VolumeOrPath, HANDLE& OutHandle,
	FStringBuilderBase& OutVolumeName, FString* OutError)
{
	OutHandle = NULL;
	OutVolumeName.Reset();

	TStringBuilder<256> AbsPath;
	FStringView VolumeName;
	if (FPathViews::IsDriveSpecifierWithoutRoot(VolumeOrPath))
	{
		// Already a drive specifier, and ToAbsolutePath does not handle a rootless drivespecifier
		VolumeName = VolumeOrPath;
	}
	else
	{
		FPathViews::ToAbsolutePath(VolumeOrPath, AbsPath);
		FStringView Remainder;
		FPathViews::SplitVolumeSpecifier(AbsPath, VolumeName, Remainder);
		if (!VolumeName.EndsWith(':'))
		{
			if (OutError)
			{
				*OutError = FString::Printf(
					TEXT("Invalid drive name in path '%s'. FileJournal is only supported for drives assigned to a drive letter (e.g. C:, D:)."),
					*AbsPath);
			}
			return EFileJournalResult::InvalidVolumeName;
		}
	}
	OutVolumeName << VolumeName;

	return CreateJournalWindowsHandle(*OutVolumeName, OutHandle, OutError);
}

void GetVolumeNameOnDisk(const TCHAR* VolumeName, FStringBuilderBase& OutVolume)
{
	OutVolume.Reset();

	constexpr int32 InitialCapacity = 64;
	TStringBuilder<InitialCapacity> Buffer;

	Buffer << VolumeName;
	// CreateFile will treat a volume name "D:" as a relative path to CWD on the drive, and we
	// want it to instead open the root path "D:\". Add on a trailing slash.
	if (Buffer.Len() > 0 && !FPathViews::IsSeparator(Buffer.GetData()[Buffer.Len() - 1]))
	{
		Buffer << '\\';
	}

	HANDLE hFile = CreateFile(*Buffer,
		FILE_READ_ATTRIBUTES, FILE_SHARE_READ | FILE_SHARE_WRITE | FILE_SHARE_DELETE,
		NULL, OPEN_EXISTING,
		FILE_FLAG_BACKUP_SEMANTICS, // Required for opening a directory, which Buffer should be.
		NULL);
	if (hFile == INVALID_HANDLE_VALUE)
	{
		OutVolume << VolumeName;
		return;
	}
	ON_SCOPE_EXIT
	{
		CloseHandle(hFile);
	};
	
	for (uint32 Length = (uint32)InitialCapacity - 1; ;)
	{
		if (Length + 1 > (uint32)Buffer.Len())
		{
			Buffer.AddUninitialized(Length + 1 - Buffer.Len());
		}
		Length = GetFinalPathNameByHandle(hFile, Buffer.GetData(),
			Buffer.Len(), // BufferSize, which must include room for a null termination character
			FILE_NAME_NORMALIZED);
		// Returned value is the required text length for the path not counting the null termination character;
		// only Min(Length+1, BufferSize) was written.
		if (Length == 0)
		{
			OutVolume << VolumeName;
			return;
		}
		if (Length < (uint32)Buffer.Len())
		{
			Buffer.RemoveSuffix(Buffer.Len() - Length);
			break;
		}
	}
	FStringView UNCPathPrefix(TEXT("\\\\?\\"));
	FString FinalPathView(Buffer);
	if (FinalPathView.StartsWith(UNCPathPrefix))
	{
		FinalPathView = FinalPathView.RightChop(UNCPathPrefix.Len());
	}
	FStringView OutVolumeView;
	FStringView Remainder;
	FPathViews::SplitVolumeSpecifier(FinalPathView, OutVolumeView, Remainder);
	OutVolume << OutVolumeView;
}

EFileJournalResult CreateJournalWindowsHandle(const TCHAR* VolumeName, HANDLE& OutHandle, FString* OutError)
{
	OutHandle = NULL;

	TStringBuilder<64> PhysicalVolumeName;
	GetVolumeNameOnDisk(VolumeName, PhysicalVolumeName);

	TStringBuilder<32> JournalName(InPlace, TEXT("\\\\.\\"), *PhysicalVolumeName);
	OutHandle = ::CreateFileW(
		*JournalName,
		FILE_TRAVERSE,
		FILE_SHARE_READ | FILE_SHARE_WRITE,
		NULL /* lpSecurityAttributes */,
		OPEN_EXISTING,
		FILE_ATTRIBUTE_NORMAL,
		NULL /* hTemplateFile */);

	if (OutHandle == INVALID_HANDLE_VALUE)
	{
		if (OutError)
		{
			uint32 LastError = FPlatformMisc::GetLastError();
			*OutError = FString::Printf(TEXT("CreateFile failed to open for reading FileJournal on volume '%s'. GetLastError == %d (0x%08x) (%s)"),
				*PhysicalVolumeName, LastError, LastError, *JournalGetLastErrorString(LastError));
		}
		return EFileJournalResult::FailedOpenJournal;
	}
	return EFileJournalResult::Success;
}

EFileJournalResult GetJournalDescriptor(HANDLE JournalWindowsHandle, const TCHAR* VolumeName,
	USN_JOURNAL_DATA_V1& OutJournalDescriptor, FString* OutError)
{
	DWORD SizeOfOutputData;
	BOOL DescribeFileJournalResult = ::DeviceIoControl(
		JournalWindowsHandle,
		FSCTL_QUERY_USN_JOURNAL,
		NULL /* InBuffer */,
		0 /* InBufferSize */,
		&OutJournalDescriptor,
		sizeof(OutJournalDescriptor),
		&SizeOfOutputData,
		NULL /* Overlapped */);
	if (!DescribeFileJournalResult)
	{
		uint32 LastError = FPlatformMisc::GetLastError();
		switch (LastError)
		{
		case ERROR_JOURNAL_NOT_ACTIVE:
			if (OutError)
			{
				// Can also create using DeviceIoControl with FSCTL_CREATE_USN_JOURNAL
				*OutError = FString::Printf(TEXT("NTFS Journal is not active for volume '%s'. ")
					TEXT("Launch cmd.exe as admin and run command `fsutil usn createJournal %s m=<SizeInBytes>`. ")
					TEXT("Recommended <SizeInBytes> is 0x40000000 (1GB)."),
					VolumeName, VolumeName);
			}
			return EFileJournalResult::JournalNotActive;
		default:
			if (OutError)
			{
				*OutError = FString::Printf(
					TEXT("FSCTL_QUERY_USN_JOURNAL failed for volume '%s'. GetLastError == %u (0x%08x) (%s)."),
					VolumeName, LastError, LastError, *JournalGetLastErrorString(LastError));
			}
			return EFileJournalResult::FailedDescribeJournal;
		}
	}
	return EFileJournalResult::Success;
}

FStringView GetFileName(FILE_ID_EXTD_DIR_INFO* Info)
{
	int32 FileNameLength = Info->FileNameLength / sizeof(Info->FileName[0]);
	return FStringView(Info->FileName, FileNameLength);
};

FStringView GetFileName(USN_RECORD_V3* Record)
{
	int32 FileNameLength = Record->FileNameLength / sizeof(Record->FileName[0]);
	return FStringView(Record->FileName, FileNameLength);
};

FString JournalGetLastErrorString(uint32 LastError)
{
	const TCHAR* ErrorName = nullptr;
	switch (LastError)
	{
	case ERROR_ACCESS_DENIED: ErrorName = TEXT("ERROR_ACCESS_DENIED"); break;
	case ERROR_BAD_LENGTH: ErrorName = TEXT("ERROR_BAD_LENGTH"); break;
	case ERROR_JOURNAL_DELETE_IN_PROGRESS: ErrorName = TEXT("ERROR_JOURNAL_DELETE_IN_PROGRESS"); break;
	case ERROR_JOURNAL_ENTRY_DELETED: ErrorName = TEXT("ERROR_JOURNAL_ENTRY_DELETED"); break;
	case ERROR_POTENTIAL_FILE_FOUND: ErrorName = TEXT("ERROR_POTENTIAL_FILE_FOUND"); break;
	case ERROR_JOURNAL_NOT_ACTIVE: ErrorName = TEXT("ERROR_JOURNAL_NOT_ACTIVE"); break;
	default: break;
	}

	TCHAR ErrorBuffer[256];
	FPlatformMisc::GetSystemErrorMessage(ErrorBuffer, UE_ARRAY_COUNT(ErrorBuffer), static_cast<int32>(LastError));
	if (ErrorName && ErrorBuffer[0] != '\0')
	{
		return FString::Printf(TEXT("%s: %s"), ErrorName, ErrorBuffer);
	}
	else if (ErrorName)
	{
		return ErrorName;
	}
	else if (ErrorBuffer[0] != '\0')
	{
		return FString(ErrorBuffer);
	}

	return TEXT("<UnknownErrorCode>");
}

FFileJournalFileHandle ToFileHandle(FILE_ID_128& ReferenceNumber)
{
	static_assert(sizeof(ReferenceNumber) == 16);
	static_assert((int)ExtendedFileIdType < 256);
	// In version 3, the FileReferenceNumbers are FILE_ID_128, and their FILE_ID_DESCRIPTOR has Type ExtendedFileType
	// and ExtendedFileId equal to the FileReferenceNumber
	FFileJournalFileHandle Result;
	Result.Bytes[0] = (uint8)ExtendedFileIdType;
	Result.Bytes[1] = 0;
	Result.Bytes[2] = 0;
	Result.Bytes[3] = 1 << 7; // Valid = true
	FMemory::Memcpy(&Result.Bytes[4], &ReferenceNumber, sizeof(ReferenceNumber));
	return Result;
}

FFileJournalFileHandle ToFileHandle(uint64 ReferenceNumber)
{
	static_assert(sizeof(ReferenceNumber) == 8);
	static_assert((int)FileIdType < 256);
	// In version 2, the FileReferenceNumbers are DWORDDLONG (aka uint64), and their FILE_ID_DESCRIPTOR has type FileIdType
	// and FileId.QuadPart equal to their FileReferenceNumber
	FFileJournalFileHandle Result;
	Result.Bytes[0] = (uint8)FileIdType;
	Result.Bytes[1] = 0;
	Result.Bytes[2] = 0;
	Result.Bytes[3] = 1 << 7; // Valid = true
	FMemory::Memcpy(&Result.Bytes[4], &ReferenceNumber, sizeof(ReferenceNumber));
	Result.Bytes[12] = 0;
	Result.Bytes[13] = 0;
	Result.Bytes[14] = 0;
	Result.Bytes[15] = 0;
	return Result;
}

} // namespace UE::WindowsPlatformFileJournal::Private

#else // UE_WINDOWS_PLATFORM_FILEJOURNAL_ENABLED 

#if !WITH_EDITOR
namespace UE::WindowsPlatformFileJournal::Private
{
constexpr const TCHAR* OnlyAvailableInEditorMessage = TEXT("FileJournal is only available in editor.");
}
#endif

bool FWindowsPlatformFile::FileJournalIsAvailable(const TCHAR* VolumeOrPath, ELogVerbosity::Type* OutErrorLevel,
	FString* OutError)
{
	using namespace UE::WindowsPlatformFileJournal::Private;

#if !WITH_EDITOR
	if (OutErrorLevel)
	{
		*OutErrorLevel = ELogVerbosity::Display;
		OutErrorLevel = nullptr;
	}
	if (OutError)
	{
		*OutError = OnlyAvailableInEditorMessage;
		OutError = nullptr;
	}
#endif
	return IPlatformFile::FileJournalIsAvailable(VolumeOrPath, OutErrorLevel, OutError);
}

uint64 FWindowsPlatformFile::FileJournalGetMaximumSize(const TCHAR* VolumeOrPath, ELogVerbosity::Type* OutErrorLevel,
	FString* OutError) const
{
	using namespace UE::WindowsPlatformFileJournal::Private;

#if !WITH_EDITOR
	if (OutErrorLevel)
	{
		*OutErrorLevel = ELogVerbosity::Display;
		OutErrorLevel = nullptr;
	}
	if (OutError)
	{
		*OutError = OnlyAvailableInEditorMessage;
		OutError = nullptr;
	}
#endif

	return IPlatformFile::FileJournalGetMaximumSize(VolumeOrPath, OutErrorLevel, OutError);
}

EFileJournalResult  FWindowsPlatformFile::FileJournalGetLatestEntry(const TCHAR* VolumeName,
	FFileJournalId& OutJournalId, FFileJournalEntryHandle& OutEntryHandle, FString* OutError)
{
	using namespace UE::WindowsPlatformFileJournal::Private;

#if !WITH_EDITOR
	if (OutError)
	{
		*OutError = OnlyAvailableInEditorMessage;
		OutError = nullptr;
	}
#endif
	return IPlatformFile::FileJournalGetLatestEntry(VolumeName, OutJournalId, OutEntryHandle, OutError);
}

bool FWindowsPlatformFile::FileJournalIterateDirectory(const TCHAR* Directory,
	FDirectoryJournalVisitorFunc Visitor, FString* OutError)
{
	return IPlatformFile::FileJournalIterateDirectory(Directory, Visitor, OutError);
}

FFileJournalData FWindowsPlatformFile::FileJournalGetFileData(const TCHAR* FilenameOrDirectory, FString* OutError)
{
	return IPlatformFile::FileJournalGetFileData(FilenameOrDirectory, OutError);
}

EFileJournalResult FWindowsPlatformFile::FileJournalReadModified(const TCHAR* VolumeName,
	const FFileJournalId& JournalIdOfStartingEntry, const FFileJournalEntryHandle& StartingJournalEntry,
	TMap<FFileJournalFileHandle, FString>& KnownDirectories, TSet<FString>& OutModifiedDirectories,
	FFileJournalEntryHandle& OutNextJournalEntry, FString* OutError)
{
	using namespace UE::WindowsPlatformFileJournal::Private;

#if !WITH_EDITOR
	if (OutError)
	{
		*OutError = OnlyAvailableInEditorMessage;
		OutError = nullptr;
	}
#endif
	return IPlatformFile::FileJournalReadModified(VolumeName, JournalIdOfStartingEntry, StartingJournalEntry,
		KnownDirectories, OutModifiedDirectories, OutNextJournalEntry, OutError);
}

#endif // else !UE_WINDOWS_PLATFORM_FILEJOURNAL_ENABLED
