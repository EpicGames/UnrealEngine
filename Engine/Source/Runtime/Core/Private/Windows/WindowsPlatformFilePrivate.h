// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Windows/WindowsPlatformFile.h"

#include "Containers/StringView.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "Misc/DateTime.h"
#include "Misc/StringBuilder.h"
#include "Windows/WindowsHWrapper.h"

#define USE_WINDOWS_ASYNC_IMPL 0
#define USE_OVERLAPPED_IO (!IS_PROGRAM && !WITH_EDITOR)		// Use straightforward synchronous I/O in cooker/editor

/**
 * Windows File I/O implementation
**/
class FWindowsPlatformFile : public IPhysicalPlatformFile
{
public:
	/**
	  * Convert from a valid Unreal Path to a canonical and strict-valid Windows Path.
	  * An Unreal Path may have either \ or / and may have empty directories (two / in a row), and may have .. and may be relative
	  * A canonical and strict-valid Windows Path has only \, does not have .., does not have empty directories, and is an absolute path, either \\UNC or D:\
	  * We need to use strict-valid Windows Paths when calling Windows API calls so that we can support the long-path prefix \\?\
	  */
	static void NormalizeWindowsPath(FStringBuilderBase& Path, bool bIsFilename);

	class FNormalizedFilename : public TStringBuilder<256>
	{
	public:
		explicit FNormalizedFilename(const TCHAR* Filename);
		explicit FNormalizedFilename(FStringView Filename);
		explicit FNormalizedFilename(FStringView Dir, FStringView Filename);
	};

	class FNormalizedDirectory : public TStringBuilder<256>
	{
	public:
		explicit FNormalizedDirectory(const TCHAR* Directory);
	};

public:
	//~ For visibility of overloads we don't override
	using IPhysicalPlatformFile::IterateDirectory;
	using IPhysicalPlatformFile::IterateDirectoryStat;

	virtual bool FileExists(const TCHAR* Filename) override;
	virtual int64 FileSize(const TCHAR* Filename) override;
	virtual bool DeleteFile(const TCHAR* Filename) override;
	virtual bool IsReadOnly(const TCHAR* Filename) override;
	virtual bool MoveFile(const TCHAR* To, const TCHAR* From) override;
	virtual bool SetReadOnly(const TCHAR* Filename, bool bNewReadOnlyValue) override;
	virtual FDateTime GetTimeStamp(const TCHAR* Filename) override;
	virtual void SetTimeStamp(const TCHAR* Filename, FDateTime DateTime) override;
	virtual FDateTime GetAccessTimeStamp(const TCHAR* Filename) override;
	virtual FString GetFilenameOnDisk(const TCHAR* Filename) override;
	virtual ESymlinkResult IsSymlink(const TCHAR* Filename) override;
	virtual bool HasMarkOfTheWeb(FStringView Filename, FString* OutSourceURL = nullptr) override;
	virtual bool SetMarkOfTheWeb(FStringView Filename, bool bNewStatus, const FString* InSourceURL = nullptr) override;

#if USE_WINDOWS_ASYNC_IMPL
	virtual IAsyncReadFileHandle* OpenAsyncRead(const TCHAR* Filename, bool bAllowWrite = false) override;
#endif

	virtual FFileOpenResult OpenRead(const TCHAR* Filename, EOpenReadFlags Flags) override;
	virtual IFileHandle* OpenRead(const TCHAR* Filename, bool bAllowWrite = false) override;
	virtual FFileOpenResult OpenReadNoBuffering(const TCHAR* Filename, EOpenReadFlags Flags) override;
	virtual IFileHandle* OpenReadNoBuffering(const TCHAR* Filename, bool bAllowWrite = false) override;
	virtual IFileHandle* OpenWrite(const TCHAR* Filename, bool bAppend = false, bool bAllowRead = false) override;
	virtual FOpenMappedResult OpenMappedEx(const TCHAR* Filename, EOpenReadFlags OpenOptions = EOpenReadFlags::None, int64 MaximumSize = 0) override;
	virtual FOpenMappedResult OpenMappedEx2(const TCHAR* Filename, EOpenMappedFlags OpenOptions = EOpenMappedFlags::None, int64 MaximumSize = 0) override;
	virtual bool DirectoryExists(const TCHAR* Directory) override;
	virtual bool CreateDirectory(const TCHAR* Directory) override;
	virtual bool DeleteDirectory(const TCHAR* Directory) override;
	virtual FFileStatData GetStatData(const TCHAR* FilenameOrDirectory) override;
	virtual bool IterateDirectory(const TCHAR* Directory, FDirectoryVisitor& Visitor) override;
	virtual bool IterateDirectoryStat(const TCHAR* Directory, FDirectoryStatVisitor& Visitor) override;

	// Forced not inline to reduce stack space usage since IterateDirectoryCommon might be recursive
	FORCENOINLINE static HANDLE FindFirstFileWithWildcard(const TCHAR* Directory, WIN32_FIND_DATAW& OutData);
	bool IterateDirectoryCommon(const TCHAR* Directory, const TFunctionRef<bool(const WIN32_FIND_DATAW&)>& Visitor);

	virtual bool FileJournalIsAvailable(const TCHAR* VolumeOrPath = nullptr, ELogVerbosity::Type* OutErrorLevel = nullptr,
		FString* OutError = nullptr) override;
	virtual uint64 FileJournalGetMaximumSize(const TCHAR* VolumeOrPath, ELogVerbosity::Type* OutErrorLevel = nullptr,
		FString* OutError = nullptr) const override;
	virtual EFileJournalResult FileJournalGetLatestEntry(const TCHAR* VolumeName, FFileJournalId& OutJournalId, 
		FFileJournalEntryHandle& OutEntryHandle, FString* OutError = nullptr) override;
	virtual bool FileJournalIterateDirectory(const TCHAR* Directory, FDirectoryJournalVisitorFunc Visitor, FString* OutError = nullptr) override;
	virtual FFileJournalData FileJournalGetFileData(const TCHAR* FilenameOrDirectory, FString* OutError = nullptr) override;
	virtual EFileJournalResult FileJournalReadModified(const TCHAR* VolumeName,
		const FFileJournalId& JournalIdOfStartingEntry, const FFileJournalEntryHandle& StartingJournalEntry,
		TMap<FFileJournalFileHandle, FString>& KnownDirectories, TSet<FString>& OutModifiedDirectories,
		FFileJournalEntryHandle& OutNextJournalEntry, FString* OutError = nullptr) override;
	virtual FString FileJournalGetVolumeName(FStringView InPath) override;
};

namespace UE::WindowsPlatformFile::Private
{

int32 UEDayOfWeekToWindowsSystemTimeDayOfWeek(const EDayOfWeek InDayOfWeek);
FDateTime WindowsFileTimeToUEDateTime(const FILETIME& InFileTime);
FILETIME UEDateTimeToWindowsFileTime(const FDateTime& InDateTime);

} // namespace UE::WindowsPlatformFile::Private