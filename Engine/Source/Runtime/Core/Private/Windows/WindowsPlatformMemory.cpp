// Copyright Epic Games, Inc. All Rights Reserved.

#include "Windows/WindowsPlatformMemory.h"

#include "Containers/UnrealString.h"
#include "CoreGlobals.h"
#include "GenericPlatform/GenericPlatformMemoryPoolStats.h"
#include "HAL/IConsoleManager.h"
#include "HAL/LowLevelMemTracker.h"
#include "HAL/MallocAnsi.h"
#include "HAL/MallocBinned.h"
#include "HAL/MallocBinned2.h"
#include "HAL/MallocBinned3.h"
#include "HAL/MallocDoubleFreeFinder.h"
#include "HAL/MallocMimalloc.h"
#include "HAL/MallocLibpas.h"
#include "HAL/MallocStomp.h"
#include "HAL/MallocStomp2.h"
#include "HAL/MallocTBB.h"
#include "HAL/MemoryMisc.h"
#include "Logging/LogMacros.h"
#include "Math/NumericLimits.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Guid.h"
#include "Misc/OutputDevice.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Stats/Stats.h"
#include "Windows/AllowWindowsPlatformTypes.h"
#include "ProfilingDebugging/CsvProfiler.h"

#include <MemoryApi.h> // Include after AllowWindowsPlatformTypes.h

#if ENABLE_LOW_LEVEL_MEM_TRACKER && MIMALLOC_ENABLED
#include "Thirdparty/IncludeMiMalloc.h"
#include "VirtualAllocPageStatus.h"
#include <winternl.h>
#endif

#pragma warning(disable:6250)

#if ENABLE_WIN_ALLOC_TRACKING
#include <crtdbg.h>
#endif // ENABLE_WIN_ALLOC_TRACKING

#include <Psapi.h>
#pragma comment(lib, "psapi.lib")
#include <jobapi2.h>


DECLARE_MEMORY_STAT(TEXT("Windows Specific Memory Stat"),	STAT_WindowsSpecificMemoryStat, STATGROUP_MemoryPlatform);

CSV_DECLARE_CATEGORY_EXTERN(FMemory);

static int32 GWindowsPlatformMemoryGetStatsLimitTotalGB = 0;
static FAutoConsoleVariableRef CVarLogPlatformMemoryStats(
	TEXT("memory.WindowsPlatformMemoryGetStatsLimitTotalGB"),
	GWindowsPlatformMemoryGetStatsLimitTotalGB,
	TEXT("Set a synthetic platform total memory size (in GB) which will be returned as Total and Available memory from GetStats\n"),
	ECVF_Default);

static bool GWindowsUseContainerMemory = false;
static FAutoConsoleVariableRef CVarUseContainerMemory(
	TEXT("memory.WindowsPlatformMemoryUseContainerMemory"),
	GWindowsUseContainerMemory,
	TEXT("Set to assume that this process is running in a docker container and take the entire container's memory usage into consideration when computing available memory."),
	ECVF_Default);


#if ENABLE_WIN_ALLOC_TRACKING
// This allows tracking of allocations that don't happen within the engine's wrappers.
// You will probably want to set conditional breakpoints here to capture specific allocations
// which aren't related to static initialization, they will happen on the CRT anyway.
int WindowsAllocHook(int nAllocType, void *pvData,
				  size_t nSize, int nBlockUse, long lRequest,
				  const unsigned char * szFileName, int nLine )
{
	return true;
}
#endif // ENABLE_WIN_ALLOC_TRACKING

// Returns whether this process is running as part of a Windows job. Windows jobs are used to run
// docker containers and can enforce memory limits for a group of processes.
// The usual system APIs still return the memory limits of the host system, which is not correct.
static bool IsRunningAsJob()
{
	static bool bIsJob = []() {
		BOOL bIsRunningInJob = false;
		if (!IsProcessInJob(GetCurrentProcess(), NULL, &bIsRunningInJob))
		{
			DWORD ErrNo = GetLastError();
			UE_LOGF(LogHAL, Warning, "IsProcessInJob(GetCurrentProcess(), NULL, &bIsRunningInJob) failed with GetLastError() = %d",
				ErrNo
			);
			return false;
		}
		return (bool)bIsRunningInJob;
		}();
		return bIsJob;
}

void FWindowsPlatformMemory::Init()
{
	// Only allow this method to be called once
	{
		static bool bInitDone = false;
		if (bInitDone)
			return;
		bInitDone = true;
	}

	FGenericPlatformMemory::Init();

	if (IsRunningAsJob())
	{
		UE_LOGF(LogMemory, Log, "Process is running as part of a Windows Job with separate resource limits");
	}

	const FPlatformMemoryConstants& MemoryConstants = FPlatformMemory::GetConstants();
	UE_LOGF(LogMemory, Log, "Memory total: Physical=%.1fGB (%dGB approx) Virtual=%.1fGB", 
		(double)MemoryConstants.TotalPhysical/1024.0/1024.0/1024.0,
		MemoryConstants.TotalPhysicalGB, 
		(double)MemoryConstants.TotalVirtual/1024.0/1024.0/1024.0 );

	// program size is hard to ascertain and isn't so relevant on Windows. For now just set to zero.
	LLM(FLowLevelMemTracker::Get().SetProgramSize(0));

	DumpStats( *GLog );
}

FMalloc* FWindowsPlatformMemory::BaseAllocator()
{
	static FMalloc* Instance = nullptr;
	if (Instance != nullptr)
	{
		return Instance;
	}

#if ENABLE_WIN_ALLOC_TRACKING
	// This allows tracking of allocations that don't happen within the engine's wrappers.
	// This actually won't be compiled unless bDebugBuildsActuallyUseDebugCRT is set in the
	// build configuration for UBT.
	_CrtSetAllocHook(WindowsAllocHook);
#endif // ENABLE_WIN_ALLOC_TRACKING

	if (FORCE_ANSI_ALLOCATOR) //-V517
	{
		AllocatorToUse = EMemoryAllocatorToUse::Ansi;
	}
	// Mimalloc is now the default allocator for editor and programs because it has shown
	// both great performance and as much as half the memory usage of TBB after
	// heavy editor workloads. See CL 15887498 description for benchmarks.
	else if ((WITH_EDITORONLY_DATA || IS_PROGRAM || USING_INSTRUMENTATION) && MIMALLOC_ENABLED)
	{
		AllocatorToUse = EMemoryAllocatorToUse::Mimalloc;
	}
	else if ((WITH_EDITORONLY_DATA || IS_PROGRAM) && TBBMALLOC_ENABLED)
	{
		AllocatorToUse = EMemoryAllocatorToUse::TBB;
	}
	else if (USE_MALLOC_BINNED3)
	{
		AllocatorToUse = EMemoryAllocatorToUse::Binned3;
	}
	else if (USE_MALLOC_BINNED2)
	{
		AllocatorToUse = EMemoryAllocatorToUse::Binned2;
	}
	else
	{
		AllocatorToUse = EMemoryAllocatorToUse::Binned;
	}
	
#if !UE_BUILD_SHIPPING
	// If not shipping, allow overriding with command line options, this happens very early so we need to use windows functions
	const TCHAR* CommandLine = ::GetCommandLineW();

	if (FCString::Stristr(CommandLine, TEXT("-libpasmalloc")))
	{
		AllocatorToUse = EMemoryAllocatorToUse::Libpas;
	}
	else if (FCString::Stristr(CommandLine, TEXT("-ansimalloc")))
	{
		// see FPlatformMisc::GetProcessDiagnostics()
		AllocatorToUse = EMemoryAllocatorToUse::Ansi;
	}
#if TBBMALLOC_ENABLED
	else if (FCString::Stristr(CommandLine, TEXT("-tbbmalloc")))
	{
		AllocatorToUse = EMemoryAllocatorToUse::TBB;
	}
#endif
#if MIMALLOC_ENABLED
	else if (FCString::Stristr(CommandLine, TEXT("-mimalloc")))
	{
		AllocatorToUse = EMemoryAllocatorToUse::Mimalloc;
	}
#endif
	else if (FCString::Stristr(CommandLine, TEXT("-binnedmalloc3")))
	{
		AllocatorToUse = EMemoryAllocatorToUse::Binned3;
	}
	else if (FCString::Stristr(CommandLine, TEXT("-binnedmalloc2")))
	{
		AllocatorToUse = EMemoryAllocatorToUse::Binned2;
	}
	else if (FCString::Stristr(CommandLine, TEXT("-binnedmalloc")))
	{
		AllocatorToUse = EMemoryAllocatorToUse::Binned;
	}
#if WITH_MALLOC_STOMP
	else if (FCString::Stristr(CommandLine, TEXT("-stompmalloc")))
	{
		// see FPlatformMisc::GetProcessDiagnostics()
		AllocatorToUse = EMemoryAllocatorToUse::Stomp;
	}
#endif // WITH_MALLOC_STOMP
#if WITH_MALLOC_STOMP2
	if (FCString::Stristr(CommandLine, TEXT("-stomp2malloc")))
	{
		GMallocStomp2Enabled = true;
	}
#endif // WITH_MALLOC_STOMP2

	if (FCString::Stristr(CommandLine, TEXT("-doublefreefinder")))
	{
		GMallocDoubleFreeFinderEnabled = true;
	}

#endif // !UE_BUILD_SHIPPING

	switch (AllocatorToUse)
	{
	case EMemoryAllocatorToUse::Ansi:
		Instance = new FMallocAnsi();
		break;
#if WITH_MALLOC_STOMP
	case EMemoryAllocatorToUse::Stomp:
		Instance = new FMallocStomp();
		break;
#endif
#if TBBMALLOC_ENABLED
	case EMemoryAllocatorToUse::TBB:
		Instance = new FMallocTBB();
		break;
#endif
#if MIMALLOC_ENABLED
	case EMemoryAllocatorToUse::Mimalloc:
		Instance = new FMallocMimalloc();
		break;
#endif
#if LIBPASMALLOC_ENABLED
	case EMemoryAllocatorToUse::Libpas:
		Instance = new FMallocLibpas();
		break;
#endif
	case EMemoryAllocatorToUse::Binned2:
		Instance = new FMallocBinned2();
		break;
	case EMemoryAllocatorToUse::Binned3:
		Instance = new FMallocBinned3();
		break;
	default:	// intentional fall-through
	case EMemoryAllocatorToUse::Binned:
		Instance = new FMallocBinned((uint32)(GetConstants().BinnedPageSize&MAX_uint32), (uint64)MAX_uint32 + 1);
		break;
	}

	return Instance;
}

static uint64 QueryContainerCommittedMemory()
{	
	JOBOBJECT_LIMIT_VIOLATION_INFORMATION LimitInformation{};
	DWORD LengthWritten;
	if (!QueryInformationJobObject(NULL, JobObjectLimitViolationInformation, &LimitInformation, sizeof(LimitInformation), &LengthWritten))
	{
		DWORD ErrNo = GetLastError();
		UE_LOGF(LogMemory, Error, "QueryInformationJobObject(NULL, JobObjectLimitViolationInformation, &LimitInformation, sizeof(LimitInformation), &LengthWritten) failed with GetLastError() = %d",
			ErrNo
		);
		return 0;
	}
	return LimitInformation.JobMemory;
}

FPlatformMemoryStats FWindowsPlatformMemory::GetStats()
{
	// release mimalloc held pages back to OS before getting stats
	// can't do this, it calls me :
	//FMemory::Trim(true);

	/**
	 *	GlobalMemoryStatusEx 
	 *	MEMORYSTATUSEX 
	 *		ullTotalPhys
	 *		ullAvailPhys
	 *		ullTotalVirtual
	 *		ullAvailVirtual
	 *		
	 *	GetProcessMemoryInfo
	 *	PROCESS_MEMORY_COUNTERS
	 *		WorkingSetSize
	 *		UsedVirtual
	 *		PeakUsedVirtual
	 *		
	 *	GetPerformanceInfo
	 *		PPERFORMANCE_INFORMATION 
	 *		PageSize
	 * 
	 * If we are running as part of a Windows job (e.g. in a docker container), we report those limits instead.
	 */

	// This method is slow, do not call it too often.
	// #TODO Should be executed only on the background thread.

	FPlatformMemoryStats MemoryStats;

	static HANDLE MemoryResourceNotificationHandle = CreateMemoryResourceNotification(LowMemoryResourceNotification);

	// Gather platform memory stats.
	MEMORYSTATUSEX MemoryStatusEx;
	FPlatformMemory::Memzero( &MemoryStatusEx, sizeof( MemoryStatusEx ) );
	MemoryStatusEx.dwLength = sizeof( MemoryStatusEx );
	::GlobalMemoryStatusEx( &MemoryStatusEx );

	PROCESS_MEMORY_COUNTERS ProcessMemoryCounters;
	FPlatformMemory::Memzero( &ProcessMemoryCounters, sizeof( ProcessMemoryCounters ) );
	::GetProcessMemoryInfo( ::GetCurrentProcess(), &ProcessMemoryCounters, sizeof(ProcessMemoryCounters) );

	MemoryStats.TotalPhysical = MemoryStatusEx.ullTotalPhys;
	MemoryStats.AvailablePhysical = MemoryStatusEx.ullAvailPhys;
	MemoryStats.AvailableVirtual = MemoryStatusEx.ullAvailVirtual;

	// On Windows, ullTotalVirtual is artificial and represent the SKU limitation (128TB on Win10Pro) instead of the commit limit of the system we're after.
	// ullTotalPageFile represents PhysicalMemory + Disk Swap Space, which is the value we care about
	MemoryStats.TotalVirtual = MemoryStatusEx.ullTotalPageFile;

	// On Windows, Virtual Memory is limited per process to the address space (e.g. 47 bits (128Tb)), but is additionally limited by the sum of used virtual memory across all processes
	// must be less than PhysicalMemory plus the Virtual Memory Page Size. The remaining virtual memory space given this system-wide limit is stored in ullAvailPageSize
	MemoryStats.AvailableVirtual = FMath::Min(MemoryStats.AvailableVirtual, MemoryStatusEx.ullAvailPageFile);

	MemoryStats.UsedPhysical = ProcessMemoryCounters.WorkingSetSize;
	MemoryStats.PeakUsedPhysical = ProcessMemoryCounters.PeakWorkingSetSize;
	// PagefileUsage = commit charge
	MemoryStats.UsedVirtual = ProcessMemoryCounters.PagefileUsage;
	MemoryStats.PeakUsedVirtual = ProcessMemoryCounters.PeakPagefileUsage;

	BOOL bIsInLowMemoryState;
	if (MemoryResourceNotificationHandle && QueryMemoryResourceNotification(MemoryResourceNotificationHandle, &bIsInLowMemoryState))
	{
		MemoryStats.MemoryPressureStatus = bIsInLowMemoryState ? FPlatformMemoryStats::EMemoryPressureStatus::Critical :
			FPlatformMemoryStats::EMemoryPressureStatus::Nominal;
	}
	else
	{
		MemoryStats.MemoryPressureStatus = MemoryStats.FGenericPlatformMemoryStats::GetMemoryPressureStatus();
	}

	if (GWindowsUseContainerMemory && IsRunningAsJob())
	{
		const uint64 TotalUsage = QueryContainerCommittedMemory();
		ensureMsgf(TotalUsage != 0, TEXT("Estimated container memory usage is 0. This should never happen"));
		if (TotalUsage != 0)
		{
			MemoryStats.AvailableVirtual = GetConstants().TotalVirtual - TotalUsage;
		}
	}

	if ( GWindowsPlatformMemoryGetStatsLimitTotalGB > 0 )
	{
		// if GWindowsPlatformMemoryGetStatsLimitTotalGB is set
		// apply a synthetic limit to the Total memory on the system for testing
		// note the first couple of calls to this function are before the cvar for GWindowsPlatformMemoryGetStatsLimitTotalGB
		//	is processed (eg. for LLM init), and that's okay
		const uint64 TestHardLimit = (int64)GWindowsPlatformMemoryGetStatsLimitTotalGB << 30;

		// "Available" is the amount free on the whole system
		// "Used" is only what our app has used
		// the synthetic limit here pretends that our app is the only thing on the system
		//   eg. we get to use up to 100% of TestHardLimit, usage by other apps is not counted
		//   except when your real system runs out of memory

		if ( MemoryStats.UsedVirtual > TestHardLimit )
		{
			MemoryStats.TotalVirtual = MemoryStats.UsedVirtual;
			MemoryStats.AvailableVirtual = 0;
		}
		else
		{
			MemoryStats.TotalVirtual = TestHardLimit;
			MemoryStats.AvailableVirtual = FMath::Min(MemoryStats.AvailableVirtual, TestHardLimit - MemoryStats.UsedVirtual);	
		}
		
		if ( MemoryStats.UsedPhysical > TestHardLimit )
		{
			MemoryStats.TotalPhysical = MemoryStats.UsedPhysical;
			MemoryStats.AvailablePhysical = 0;
		}
		else
		{
			MemoryStats.TotalPhysical = TestHardLimit;
			MemoryStats.AvailablePhysical = FMath::Min(MemoryStats.AvailablePhysical,TestHardLimit - MemoryStats.UsedPhysical);
		}
	}

	return MemoryStats;
}

void FWindowsPlatformMemory::GetStatsForMallocProfiler( FGenericMemoryStats& out_Stats )
{
#if	STATS
	FGenericPlatformMemory::GetStatsForMallocProfiler( out_Stats );

	FPlatformMemoryStats Stats = GetStats();

	// Windows specific stats.
	out_Stats.Add( GET_STATDESCRIPTION( STAT_WindowsSpecificMemoryStat ), Stats.WindowsSpecificMemoryStat );
#endif // STATS
}

const FPlatformMemoryConstants& FWindowsPlatformMemory::GetConstants()
{
	static FPlatformMemoryConstants MemoryConstants;

	if( MemoryConstants.TotalPhysical == 0 )
	{
		// Gather platform memory constants.
		MEMORYSTATUSEX MemoryStatusEx;
		FPlatformMemory::Memzero( &MemoryStatusEx, sizeof( MemoryStatusEx ) );
		MemoryStatusEx.dwLength = sizeof( MemoryStatusEx );
		::GlobalMemoryStatusEx( &MemoryStatusEx );

		SYSTEM_INFO SystemInfo;
		FPlatformMemory::Memzero( &SystemInfo, sizeof( SystemInfo ) );
		::GetSystemInfo(&SystemInfo);

		MemoryConstants.TotalPhysical = MemoryStatusEx.ullTotalPhys;
		// On Windows, ullTotalVirtual is artificial and represent the SKU limitation (128TB on Win10Pro) instead of the commit limit of the system we're after.
		// ullTotalPageFile represents PhysicalMemory + Disk Swap Space, which is the value we care about
		MemoryConstants.TotalVirtual = MemoryStatusEx.ullTotalPageFile;
		MemoryConstants.BinnedPageSize = SystemInfo.dwAllocationGranularity;	// Use this so we get larger 64KiB pages, instead of 4KiB
		MemoryConstants.BinnedAllocationGranularity = SystemInfo.dwPageSize; // Use 4KiB pages for more efficient use of memory - 64KiB pages don't really exist on this CPU
		MemoryConstants.OsAllocationGranularity = SystemInfo.dwAllocationGranularity;	// VirtualAlloc cannot allocate memory less than that
		MemoryConstants.PageSize = SystemInfo.dwPageSize;
		MemoryConstants.AddressLimit = FPlatformMath::RoundUpToPowerOfTwo64(MemoryConstants.TotalPhysical);

		MemoryConstants.TotalPhysicalGB = (uint32)((MemoryConstants.TotalPhysical + 1024 * 1024 * 1024 - 1) / 1024 / 1024 / 1024);

		if (IsRunningAsJob())
		{
			JOBOBJECT_EXTENDED_LIMIT_INFORMATION JobInfo{};
			DWORD LengthWritten{};
			if (!QueryInformationJobObject(NULL, JobObjectExtendedLimitInformation, &JobInfo, sizeof(JobInfo), &LengthWritten))
			{
				DWORD ErrNo = GetLastError();
				UE_LOGF(LogMemory, Error, "QueryInformationJobObject(NULL, JobObjectExtendedLimitInformation, &JobInfo, sizeof(JobInfo), &LengthWritten) failed with GetLastError() = %d",
					ErrNo
				);
			}
			else
			{
				if ((JobInfo.BasicLimitInformation.LimitFlags & (JOB_OBJECT_LIMIT_JOB_MEMORY | JOB_OBJECT_LIMIT_PROCESS_MEMORY)) == 0)
				{
					UE_LOGF(LogMemory, Display, "Running as part of a job but no memory limit has been set.");
				}
				else
				{
					uint64 MemoryLimit = MemoryConstants.TotalVirtual;
					if ((JobInfo.BasicLimitInformation.LimitFlags & JOB_OBJECT_LIMIT_JOB_MEMORY) == JOB_OBJECT_LIMIT_JOB_MEMORY)
					{
						MemoryLimit = JobInfo.JobMemoryLimit;
						UE_LOGF(LogMemory, Display, "Detected a job memory limit of %.1fGB for this job.",
							(double)MemoryLimit / (1024 * 1024 * 1024)
						);
					}
					if ((JobInfo.BasicLimitInformation.LimitFlags & JOB_OBJECT_LIMIT_PROCESS_MEMORY) == JOB_OBJECT_LIMIT_PROCESS_MEMORY)
					{
						MemoryLimit = JobInfo.ProcessMemoryLimit;
						UE_LOGF(LogMemory, Display, "Detected a per-process memory limit of %.1fGB for this job.",
							(double)MemoryLimit / (1024 * 1024 * 1024)
						);
					}
					MemoryConstants.TotalVirtual = MemoryLimit;
				}
			}
		}
	}

	return MemoryConstants;	
}

bool FWindowsPlatformMemory::PageProtect(void* const Ptr, const SIZE_T Size, const bool bCanRead, const bool bCanWrite)
{
	DWORD flOldProtect;
	uint32 ProtectMode = 0;
	if (bCanRead && bCanWrite)
	{
		ProtectMode = PAGE_READWRITE;
	}
	else if (bCanWrite)
	{
		ProtectMode = PAGE_READWRITE;
	}
	else if (bCanRead)
	{
		ProtectMode = PAGE_READONLY;
	}
	else
	{
		ProtectMode = PAGE_NOACCESS;
	}
	return VirtualProtect(Ptr, Size, ProtectMode, &flOldProtect) != 0;
}
void* FWindowsPlatformMemory::BinnedAllocFromOS( SIZE_T Size )
{
#if UE_CHECK_LARGE_ALLOCATIONS
	if (UE::Memory::Private::GEnableLargeAllocationChecks)
	{
		// catch possibly erroneous large allocations
		ensureMsgf(Size <= UE::Memory::Private::GLargeAllocationThreshold,
			TEXT("Single allocation exceeded large allocation threshold"));
	}
#endif
	void* Ptr = VirtualAlloc( NULL, Size, MEM_COMMIT | MEM_RESERVE, PAGE_READWRITE );
	LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelAlloc(ELLMTracker::Platform, Ptr, Size));
	return Ptr;
}

void FWindowsPlatformMemory::BinnedFreeToOS( void* Ptr, SIZE_T Size )
{
	LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelFree(ELLMTracker::Platform, Ptr));

	CA_SUPPRESS(6001)
	// Windows maintains the size of allocation internally, so Size is unused
	verify(VirtualFree( Ptr, 0, MEM_RELEASE ) != 0);
}


size_t FWindowsPlatformMemory::FPlatformVirtualMemoryBlock::GetVirtualSizeAlignment()
{
	static SIZE_T OsAllocationGranularity = FPlatformMemory::GetConstants().OsAllocationGranularity;
	return OsAllocationGranularity;
}

size_t FWindowsPlatformMemory::FPlatformVirtualMemoryBlock::GetCommitAlignment()
{
	static SIZE_T OSPageSize = FPlatformMemory::GetConstants().PageSize;
	return OSPageSize;
}

FWindowsPlatformMemory::FPlatformVirtualMemoryBlock FWindowsPlatformMemory::FPlatformVirtualMemoryBlock::AllocateVirtual(size_t InSize, size_t InAlignment)
{
	FPlatformVirtualMemoryBlock Result;
	InSize = Align(InSize, GetVirtualSizeAlignment());
	Result.VMSizeDivVirtualSizeAlignment = InSize / GetVirtualSizeAlignment();

	size_t Alignment = FMath::Max(InAlignment, GetVirtualSizeAlignment());
	check(Alignment <= GetVirtualSizeAlignment());

	bool bTopDown = Result.GetActualSize() > 100ll * 1024 * 1024; // this is hacky, but we want to allocate huge VM blocks (like for MB3) top down

	Result.Ptr = VirtualAlloc(NULL, Result.GetActualSize(), MEM_RESERVE | (bTopDown ? MEM_TOP_DOWN : 0), PAGE_NOACCESS);


	if (!LIKELY(Result.Ptr))
	{
		FPlatformMemory::OnOutOfMemory(Result.GetActualSize(), Alignment);
	}
	check(Result.Ptr && IsAligned(Result.Ptr, Alignment));
	return Result;
}



void FWindowsPlatformMemory::FPlatformVirtualMemoryBlock::FreeVirtual()
{
	if (Ptr)
	{
		check(GetActualSize() > 0);

		CA_SUPPRESS(6001)
		// Windows maintains the size of allocation internally, so Size is unused
		verify(VirtualFree(Ptr, 0, MEM_RELEASE) != 0);

		Ptr = nullptr;
		VMSizeDivVirtualSizeAlignment = 0;
	}
}

void FWindowsPlatformMemory::FPlatformVirtualMemoryBlock::Commit(size_t InOffset, size_t InSize)
{
	check(IsAligned(InOffset, GetCommitAlignment()) && IsAligned(InSize, GetCommitAlignment()));
	check(InOffset >= 0 && InSize >= 0 && InOffset + InSize <= GetActualSize() && Ptr);

	uint8* UsePtr = ((uint8*)Ptr) + InOffset;
	if (VirtualAlloc(UsePtr, InSize, MEM_COMMIT, PAGE_READWRITE) != UsePtr)
	{
		FPlatformMemory::OnOutOfMemory(InSize, 0);
	}
}

void FWindowsPlatformMemory::FPlatformVirtualMemoryBlock::Decommit(size_t InOffset, size_t InSize)
{
	check(IsAligned(InOffset, GetCommitAlignment()) && IsAligned(InSize, GetCommitAlignment()));
	check(InOffset >= 0 && InSize >= 0 && InOffset + InSize <= GetActualSize() && Ptr);
	uint8* UsePtr = ((uint8*)Ptr) + InOffset;
	VirtualFree(UsePtr, InSize, MEM_DECOMMIT);
}





FPlatformMemory::FSharedMemoryRegion* FWindowsPlatformMemory::MapNamedSharedMemoryRegion(const FString& InName, bool bCreate, uint32 AccessMode, SIZE_T Size, const void* pSecurityAttributes)
{
	FString Name;

	// Use {Guid} as the name for the memory region without prefix.
	FGuid Guid;
	if (FGuid::ParseExact(InName, EGuidFormats::DigitsWithHyphensInBraces, Guid))
	{
		// Only the Guid string is used as the name of the memory region. It works without administrator rights
		Name = Guid.ToString(EGuidFormats::DigitsWithHyphensInBraces);
	}
	else
	{
		// The prefix "Global\\" is used to share this region with other processes. This method requires administrator rights, if pSecurityAttributes not defined
		Name = TEXT("Global\\");
		Name += InName;
	}

	DWORD OpenMappingAccess = FILE_MAP_READ;
	check(AccessMode != 0);
	if (AccessMode == FPlatformMemory::ESharedMemoryAccess::Write)
	{
		OpenMappingAccess = FILE_MAP_WRITE;
	}
	else if (AccessMode == (FPlatformMemory::ESharedMemoryAccess::Write | FPlatformMemory::ESharedMemoryAccess::Read))
	{
		OpenMappingAccess = FILE_MAP_ALL_ACCESS;
	}

	HANDLE Mapping = NULL;
	if (bCreate)
	{
		DWORD CreateMappingAccess = PAGE_READONLY;
		check(AccessMode != 0);
		if (AccessMode == FPlatformMemory::ESharedMemoryAccess::Write)
		{
			CreateMappingAccess = PAGE_WRITECOPY;
		}
		else if (AccessMode == (FPlatformMemory::ESharedMemoryAccess::Write | FPlatformMemory::ESharedMemoryAccess::Read))
		{
			CreateMappingAccess = PAGE_READWRITE;
		}

		DWORD MaxSizeHigh = (Size >> 32);
		DWORD MaxSizeLow = Size & 0xFFFFFFFF;

		Mapping = CreateFileMapping(INVALID_HANDLE_VALUE, (SECURITY_ATTRIBUTES*)pSecurityAttributes, CreateMappingAccess, MaxSizeHigh, MaxSizeLow, *Name);

		if (Mapping == NULL)
		{
			DWORD ErrNo = GetLastError();
			UE_LOGF(LogHAL, Warning, "CreateFileMapping(file=INVALID_HANDLE_VALUE, security=NULL, protect=0x%x, MaxSizeHigh=%d, MaxSizeLow=%d, name='%ls') failed with GetLastError() = %d", 
				CreateMappingAccess, MaxSizeHigh, MaxSizeLow, *Name,
				ErrNo
				);
		}
	}
	else
	{
		Mapping = OpenFileMapping(OpenMappingAccess, FALSE, *Name);

		if (Mapping == NULL)
		{
			DWORD ErrNo = GetLastError();
			UE_LOGF(LogHAL, Warning, "OpenFileMapping(access=0x%x, inherit=false, name='%ls') failed with GetLastError() = %d", 
				OpenMappingAccess, *Name,
				ErrNo
				);
		}
	}

	if (Mapping == NULL)
	{
		return NULL;
	}

	void* Ptr = MapViewOfFile(Mapping, OpenMappingAccess, 0, 0, Size);
	if (Ptr == NULL)
	{
		DWORD ErrNo = GetLastError();
		UE_LOG(LogHAL, Warning, TEXT("MapViewOfFile(mapping=0x%" UPTRINT_x_FMT ", access=0x%x, OffsetHigh=0, OffsetLow=0, NumBytes=%" SIZE_T_FMT ") failed with GetLastError() = %d"), 
			Mapping, OpenMappingAccess, Size,
			ErrNo
			);

		CloseHandle(Mapping);
		return NULL;
	}

	return new FWindowsSharedMemoryRegion(Name, AccessMode, Ptr, Size, Mapping);
}

bool FWindowsPlatformMemory::UnmapNamedSharedMemoryRegion(FSharedMemoryRegion * MemoryRegion)
{
	bool bAllSucceeded = true;

	if (MemoryRegion)
	{
		FWindowsSharedMemoryRegion * WindowsRegion = static_cast< FWindowsSharedMemoryRegion* >( MemoryRegion );

		if (!UnmapViewOfFile(WindowsRegion->GetAddress()))
		{
			bAllSucceeded = false;

			int ErrNo = GetLastError();
			UE_LOGF(LogHAL, Warning, "UnmapViewOfFile(address=%p) failed with GetLastError() = %d", 
				WindowsRegion->GetAddress(),
				ErrNo
				);
		}

		if (!CloseHandle(WindowsRegion->GetMapping()))
		{
			bAllSucceeded = false;

			int ErrNo = GetLastError();
			UE_LOG(LogHAL, Warning, TEXT("CloseHandle(handle=0x%" UPTRINT_x_FMT ") failed with GetLastError() = %d"), 
				WindowsRegion->GetMapping(),
				ErrNo
				);
		}

		// delete the region
		delete WindowsRegion;
	}

	return bAllSucceeded;
}

void FWindowsPlatformMemory::InternalUpdateStats( const FPlatformMemoryStats& MemoryStats )
{
	// Windows specific stats.
	SET_MEMORY_STAT( STAT_WindowsSpecificMemoryStat, MemoryStats.WindowsSpecificMemoryStat );
}

/**
* LLM uses these low level functions (LLMAlloc and LLMFree) to allocate memory. It grabs
* the function pointers by calling FPlatformMemory::GetLLMAllocFunctions. If these functions
* are not implemented GetLLMAllocFunctions should return false and LLM will be disabled.
*/

#if ENABLE_LOW_LEVEL_MEM_TRACKER

int64 LLMMallocTotal = 0;

const size_t LLMPageSize = 4096;

void* LLMAlloc(size_t Size)
{
	size_t AlignedSize = Align(Size, LLMPageSize);

	off_t DirectMem = 0;
	void* Addr = VirtualAlloc(NULL, Size, MEM_COMMIT | MEM_RESERVE, PAGE_READWRITE);

	check(Addr);

	LLMMallocTotal += static_cast<int64>(AlignedSize);

	return Addr;
}

void LLMFree(void* Addr, size_t Size)
{
	VirtualFree(Addr, 0, MEM_RELEASE);

	size_t AlignedSize = Align(Size, LLMPageSize);
	LLMMallocTotal -= static_cast<int64>(AlignedSize);
}


bool FWindowsPlatformMemory::GetLLMAllocFunctions(void*(*&OutAllocFunction)(size_t), void(*&OutFreeFunction)(void*, size_t), int32& OutAlignment)
{
	OutAllocFunction = LLMAlloc;
	OutFreeFunction = LLMFree;
	OutAlignment = LLMPageSize;

	return true;
}

#if MIMALLOC_ENABLED

FVirtualAllocPageStatus& GetVirtualAllocPageStatus()
{
	static FVirtualAllocPageStatus PageBits;
	return PageBits;
}

LPVOID __stdcall MiMallocVirtualAlloc(LPVOID lpAddress, SIZE_T dwSize, DWORD flAllocationType, DWORD flProtect)
{
	LPVOID Result = VirtualAlloc(lpAddress, dwSize, flAllocationType, flProtect);
	if (Result && FLowLevelMemTracker::IsEnabled())
	{
		if (flAllocationType & MEM_RESERVE)
		{
			SIZE_T dwOldSize;
			GetVirtualAllocPageStatus().AddReservationSize(Result, dwSize, dwOldSize);
			LLMCheck(dwOldSize == 0 || dwSize == dwOldSize);
		}
		if (flAllocationType & MEM_COMMIT)
		{
			int64 DeltaSize = GetVirtualAllocPageStatus().MarkChangedAndReturnDeltaSize(Result, dwSize, true /* bCommitted */);
			LLM_PLATFORM_SCOPE(ELLMTag::FMalloc);
			FLowLevelMemTracker::Get().OnLowLevelChangeInMemoryUse(ELLMTracker::Platform, DeltaSize);
		}
	}
	return Result;
}

BOOL __stdcall MiMallocVirtualFree(LPVOID lpAddress, SIZE_T dwSize, DWORD dwFreeType)
{
	if (dwFreeType & (MEM_DECOMMIT | MEM_RELEASE) && FLowLevelMemTracker::IsEnabled())
	{
		SIZE_T AllocationSize = dwSize;
		if (dwFreeType & MEM_RELEASE)
		{
			LLMCheck(AllocationSize == 0);
			AllocationSize = GetVirtualAllocPageStatus().GetAndRemoveReservationSize(lpAddress);
			LLMCheck(AllocationSize > 0);
		}
		int64 DeltaSize = GetVirtualAllocPageStatus().MarkChangedAndReturnDeltaSize(lpAddress, AllocationSize, false /* bCommitted */);
		LLM_PLATFORM_SCOPE(ELLMTag::FMalloc);
		FLowLevelMemTracker::Get().OnLowLevelChangeInMemoryUse(ELLMTracker::Platform, DeltaSize);
	}
	return VirtualFree(lpAddress, dwSize, dwFreeType);
}

typedef PVOID(__stdcall* VirtualAlloc2Func)(HANDLE Process, PVOID BaseAddress, SIZE_T Size, ULONG AllocationType, ULONG PageProtection,
	void* ExtendedParameters, ULONG ParameterCount);
VirtualAlloc2Func GOSVirtualAlloc2 = nullptr;
PVOID __stdcall MiMallocVirtualAlloc2(HANDLE Process, PVOID BaseAddress, SIZE_T Size, ULONG AllocationType, ULONG PageProtection,
	void* ExtendedParameters, ULONG ParameterCount)
{
	PVOID Result = GOSVirtualAlloc2(Process, BaseAddress, Size, AllocationType, PageProtection, ExtendedParameters, ParameterCount);
	if (Result && FLowLevelMemTracker::IsEnabled())
	{
		if (AllocationType & MEM_RESERVE)
		{
			SIZE_T OldSize;
			GetVirtualAllocPageStatus().AddReservationSize(Result, Size, OldSize);
			LLMCheck(OldSize == 0 || OldSize == Size);
		}
		if (AllocationType & MEM_COMMIT)
		{
			int64 DeltaSize = GetVirtualAllocPageStatus().MarkChangedAndReturnDeltaSize(Result, Size, true /* bCommitted */);
			LLM_PLATFORM_SCOPE(ELLMTag::FMalloc);
			FLowLevelMemTracker::Get().OnLowLevelChangeInMemoryUse(ELLMTracker::Platform, DeltaSize);
		}
	}
	return Result;
}

VirtualAlloc2Func GetMiMallocVirtualAlloc2()
{
	GOSVirtualAlloc2 = nullptr;
	HINSTANCE hDll;
	hDll = LoadLibrary(TEXT("kernelbase.dll"));
	if (hDll != NULL)
	{
		// warning C4191: unsafe conversion from 'FARPROC' to void (__cdecl *)(void)
		// We avoid this warning by casting the FARPROC to void* before casting it to our desired function prototype

		// use VirtualAlloc2FromApp if possible as it is available to Windows store apps
		GOSVirtualAlloc2 = (VirtualAlloc2Func)(void *)GetProcAddress(hDll, "VirtualAlloc2FromApp");
		if (GOSVirtualAlloc2 == nullptr) GOSVirtualAlloc2 = (VirtualAlloc2Func)(void *)GetProcAddress(hDll, "VirtualAlloc2");
		FreeLibrary(hDll);
	}
	return GOSVirtualAlloc2 ? MiMallocVirtualAlloc2 : nullptr;
}

typedef NTSTATUS(__stdcall* NtAllocateVirtualMemoryExFunc)(HANDLE Process, PVOID* BaseAddress, SIZE_T* Size, ULONG AllocationType,
	ULONG PageProtection, PVOID ExtendedParameters, ULONG ParameterCount);
NtAllocateVirtualMemoryExFunc GOSNtAllocateVirtualMemoryEx = nullptr;
NTSTATUS __stdcall MiMallocNtAllocateVirtualMemoryEx(HANDLE Process, PVOID* BaseAddress, SIZE_T* Size, ULONG AllocationType,
	ULONG PageProtection, PVOID ExtendedParameters, ULONG ParameterCount)
{
	NTSTATUS Result = GOSNtAllocateVirtualMemoryEx(Process, BaseAddress, Size, AllocationType, PageProtection, ExtendedParameters, ParameterCount);
	if (*BaseAddress && FLowLevelMemTracker::IsEnabled())
	{
		if (AllocationType & MEM_RESERVE)
		{
			SIZE_T OldSize;
			GetVirtualAllocPageStatus().AddReservationSize(*BaseAddress, *Size, OldSize);
			LLMCheck(OldSize == 0 || OldSize == *Size);
		}
		if (AllocationType & MEM_COMMIT)
		{
			int64 DeltaSize = GetVirtualAllocPageStatus().MarkChangedAndReturnDeltaSize(*BaseAddress, *Size, true /* bCommitted */);
			LLM_PLATFORM_SCOPE(ELLMTag::FMalloc);
			FLowLevelMemTracker::Get().OnLowLevelChangeInMemoryUse(ELLMTracker::Platform, DeltaSize);
		}
	}
	return Result;
}

NtAllocateVirtualMemoryExFunc GetMiMallocNtAllocateVirtualMemoryEx()
{
	GOSNtAllocateVirtualMemoryEx = nullptr;
	HINSTANCE hDll = LoadLibrary(TEXT("ntdll.dll"));
	if (hDll != NULL) {
		// warning C4191: unsafe conversion from 'FARPROC' to void (__cdecl *)(void)
		// We avoid this warning by casting the FARPROC to void* before casting it to our desired function prototype
		GOSNtAllocateVirtualMemoryEx = (NtAllocateVirtualMemoryExFunc)(void *)GetProcAddress(hDll, "NtAllocateVirtualMemoryEx");
		FreeLibrary(hDll);
	}
	return GOSNtAllocateVirtualMemoryEx ? MiMallocNtAllocateVirtualMemoryEx : nullptr;
}

#endif // MIMALLOC_ENABLED

#endif // ENABLE_LOW_LEVEL_MEM_TRACKER

void FWindowsPlatformMemory::MiMallocInit()
{
#if ENABLE_LOW_LEVEL_MEM_TRACKER && MIMALLOC_ENABLED
	mi_ext_win32_allocators_t allocators;
	allocators.VirtualAlloc = MiMallocVirtualAlloc;
	allocators.VirtualFree = MiMallocVirtualFree;
	allocators.VirtualAlloc2 = GetMiMallocVirtualAlloc2();
	allocators.NtAllocateVirtualMemoryEx = GetMiMallocNtAllocateVirtualMemoryEx();

	mi_ext_set_win32_allocators(&allocators);
#endif
}

#include "Windows/HideWindowsPlatformTypes.h"
