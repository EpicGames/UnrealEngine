// Copyright Epic Games, Inc. All Rights Reserved.

#include "Windows/WindowsPlatformMisc.h"
#include "Misc/DateTime.h"
#include "Misc/AssertionMacros.h"
#include "Logging/LogMacros.h"
#include "Misc/OutputDevice.h"
#include "HAL/PlatformStackWalk.h"
#include "HAL/PlatformProcess.h"
#include "HAL/UnrealMemory.h"
#include "Templates/UnrealTemplate.h"
#include "CoreGlobals.h"
#include "HAL/FileManager.h"
#include "Misc/CString.h"
#include "Misc/Parse.h"
#include "Misc/MessageDialog.h"
#include "Containers/StringConv.h"
#include "Containers/UnrealString.h"
#include "CoreGlobals.h"
#include "Misc/CommandLine.h"
#include "Misc/Paths.h"
#include "Internationalization/Text.h"
#include "Math/Color.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Misc/OutputDeviceFile.h"
#include "Misc/OutputDeviceError.h"
#include "Misc/FeedbackContext.h"
#include "Misc/CoreDelegates.h"
#include "Misc/App.h"
#include "Misc/ScopeExit.h"
#include "HAL/ExceptionHandling.h"
#include "Misc/SecureHash.h"
#include "HAL/IConsoleManager.h"
#include "Misc/EngineVersion.h"
#include "GenericPlatform/GenericPlatformCrashContext.h"
#include "Windows/WindowsPlatformCrashContext.h"
#include "HAL/Platform.h"
#include "HAL/PlatformOutputDevices.h"
#include "AutoRTFM.h"
#if WITH_APPLICATION_CORE
#include "Windows/WindowsConfigRules.h"
#endif // WITH_APPLICATION_CORE

#include "GenericPlatform/GenericPlatformChunkInstall.h"
#include "GenericPlatform/GenericPlatformDriver.h"
#include "HAL/ThreadHeartBeat.h"
#include "ProfilingDebugging/ExternalProfiler.h"
#include "Async/TaskGraphInterfaces.h"

// Resource includes.
#include "Runtime/Launch/Resources/Windows/resource.h"

#include "Windows/AllowWindowsPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
	#include <time.h>
	#include <mmsystem.h>
	#include <rpcsal.h>
	#include <gameux.h>
	#include <ShlObj.h>
	#include <IntShCut.h>
	#include <shellapi.h>
	#include <shlwapi.h>
	#include <IPHlpApi.h>
	#include <VersionHelpers.h>
	#include <netlistmgr.h>
#include "Windows/AllowWindowsPlatformAtomics.h"
	#include <comdef.h>
	#include <Wbemidl.h>
#include "Windows/HideWindowsPlatformAtomics.h"
THIRD_PARTY_INCLUDES_END
#include "Windows/HideWindowsPlatformTypes.h"

#include "Modules/ModuleManager.h"

#include "Windows/AllowWindowsPlatformTypes.h"
	#include <Psapi.h>
#include "Windows/HideWindowsPlatformTypes.h"
#pragma comment(lib, "psapi.lib")
#pragma comment(lib, "wbemuuid.lib")

#include <fcntl.h>
#include <io.h>

#if PLATFORM_COMPILER_CLANG && PLATFORM_CPU_X86_FAMILY
	#include <cpuid.h>
#else
	#include <intrin.h>
#endif // PLATFORM_COMPILER_CLANG

#include "Windows/AllowWindowsPlatformTypes.h"

#include "FramePro/FrameProProfiler.h"

// Collect extended storage information is only enabled if target has explicitly enabled it
// Reason is because we've seen 6+ minutes stalls in the dcom ConnectServer call when compiling shaders
#ifndef ALLOW_COLLECT_STORAGE_INFORMATION
#define ALLOW_COLLECT_STORAGE_INFORMATION 0
#endif

// Collect extended storage information asynchronously. A Sync call can take in the order of 1 second on the calling thread
bool GCollectStorageInformationAsync = false;
static FAutoConsoleVariableRef CVarCollectStorageInformationAsync(
	TEXT("g.CollectStorageInformationAsync"),
	GCollectStorageInformationAsync,
	TEXT("Collects extended storage information async"),
	ECVF_ReadOnly);

// This might not be defined by Windows when maintaining backwards-compatibility to pre-Win8 builds
#ifndef SM_CONVERTIBLESLATEMODE
#define SM_CONVERTIBLESLATEMODE			0x2003
#endif

// this cvar can be removed once we have a single method that works well
static TAutoConsoleVariable<int32> CVarDriverDetectionMethod(
	TEXT("r.DriverDetectionMethod"),
	5,
	TEXT("Defines which implementation is used to detect the GPU driver (to check for old drivers, logs and statistics)\n"
	     "  0: Iterate available drivers in registry and choose the one with the same name, if in question use next method (happens)\n"
	     "  1: Get the driver of the primary adapter (might not be correct when dealing with multiple adapters)\n"
	     "  2: Use DirectX LUID (would be the best, not yet implemented)\n"
	     "  3: Use Windows functions, use the primary device (might be wrong when API is using another adapter)\n"
	     "  4: Use Windows functions, use names such as DirectX Device (newest, most promising)\n"
	     "  5: Use Windows SetupAPI functions"),
	ECVF_RenderThreadSafe);

#if PLATFORM_COMPILER_CLANG && PLATFORM_CPU_X86_FAMILY
	#define UE_CPUID(Info, FunctionID) __cpuid(FunctionID, Info[0], Info[1], Info[2], Info[3]);
#else
	#define UE_CPUID(Info, FunctionID) __cpuid(Info, FunctionID);
#endif //PLATFORM_COMPILER_CLANG

namespace UE::Core::Private
{
	static TAutoConsoleVariable<bool> CVarDisableWindowsMessageBox(
		TEXT("WindowsPlatform.DisableMessageBox"),
		false,
		TEXT("If this is true, no MessageBox will be displayed and calls to MessageBoxExt will return EAppReturnType::Cancel by default."),
		ECVF_Default);
	
	static TAutoConsoleVariable<bool> CVarWindowsRaiseExceptionsOnFaultTermination(
		TEXT("WindowsPlatform.RaiseExceptionOnFaultTermination"),
		false,
		TEXT("If true, regular faults will raise an exception (1) prior to terminating the process, useful for external crash handlers"),
		ECVF_Default);
}

int32 GetOSVersionsHelper( TCHAR* OutOSVersionLabel, int32 OSVersionLabelLength, TCHAR* OutOSSubVersionLabel, int32 OSSubVersionLabelLength )
{
	int32 ErrorCode = (int32)FWindowsOSVersionHelper::SUCCEEDED;

	// Get system info
	SYSTEM_INFO SystemInfo;
	GetNativeSystemInfo( &SystemInfo );

	OSVERSIONINFOEX OsVersionInfo = {0};
	OsVersionInfo.dwOSVersionInfoSize = sizeof( OSVERSIONINFOEX );
	FString OSVersionLabel = "Windows (unknown version)";
	FString OSSubVersionLabel;
#ifdef __clang__
#pragma clang diagnostic push
#pragma clang diagnostic ignored "-Wdeprecated-declarations"
#else
#pragma warning(push)
#pragma warning(disable : 4996) // 'function' was declared deprecated
#endif
	CA_SUPPRESS(28159)
	if( GetVersionEx( (LPOSVERSIONINFO)&OsVersionInfo ) )
#ifdef __clang__
#pragma clang diagnostic pop
#else
#pragma warning(pop)
#endif
	{
		bool bIsInvalidVersion = false;

		switch (OsVersionInfo.dwMajorVersion)
		{
		case 6:
			switch (OsVersionInfo.dwMinorVersion)
			{
			case 1:
				if (OsVersionInfo.wProductType == VER_NT_WORKSTATION)
				{
					OSVersionLabel = TEXT("Windows 7");
				}
				else
				{
					OSVersionLabel = TEXT("Windows Server 2008 R2");
				}
				break;
			case 2:
				if (OsVersionInfo.wProductType == VER_NT_WORKSTATION)
				{
					OSVersionLabel = TEXT("Windows 8");
				}
				else
				{
					OSVersionLabel = TEXT("Windows Server 2012");
				}
				break;
			case 3:
				if (OsVersionInfo.wProductType == VER_NT_WORKSTATION)
				{
					OSVersionLabel = TEXT("Windows 8.1");
				}
				else
				{
					OSVersionLabel = TEXT("Windows Server 2012 R2");
				}
				break;
			default:
				ErrorCode |= (int32)FWindowsOSVersionHelper::ERROR_UNKNOWNVERSION;
				break;
			}
			break;
		case 10:
			switch (OsVersionInfo.dwMinorVersion)
			{
			case 0:
				if (OsVersionInfo.wProductType == VER_NT_WORKSTATION)
				{
					// Windows 11 still reports a major version of 10 and minor of 0, so it looks
					// like we need to use the build number as the discriminator
					if (OsVersionInfo.dwBuildNumber >= 22000)
					{
						OSVersionLabel = TEXT("Windows 11");
					}
					else
					{
						OSVersionLabel = TEXT("Windows 10");
					}
				}
				else
				{
					// Same thing here, Windows Server 2019 and 2022 both claim to be version 10.0, so use the
					// build number to decide.
					if (OsVersionInfo.dwBuildNumber >= 20348)
					{
						OSVersionLabel = TEXT("Windows Server 2022");
					}
					else
					{
						OSVersionLabel = TEXT("Windows Server 2019");
					}
				}

				// For Windows 10, get the release number and append that to the string too (eg. 1709 = Fall Creators Update). 
				// There doesn't seem to be any good way to get this other than grabbing an entry from the registry.
				// 
				// The new semi-annual release scheme 20H1/20H2 etc appears to use a different key so we query that first.
				{
					FString DisplayVersion;
					if (FWindowsPlatformMisc::QueryRegKey(HKEY_LOCAL_MACHINE, TEXT("SOFTWARE\\Microsoft\\Windows NT\\CurrentVersion"), TEXT("DisplayVersion"), DisplayVersion))
					{
						OSVersionLabel += FString::Printf(TEXT(" (%s)"), *DisplayVersion);
					}
					else
					{
						FString ReleaseId;
						if (FWindowsPlatformMisc::QueryRegKey(HKEY_LOCAL_MACHINE, TEXT("SOFTWARE\\Microsoft\\Windows NT\\CurrentVersion"), TEXT("ReleaseId"), ReleaseId))
						{
							OSVersionLabel += FString::Printf(TEXT(" (Release %s)"), *ReleaseId);
						}
					}

					FString UpdateBuildRevision;
					if (!FWindowsPlatformMisc::QueryRegKey(HKEY_LOCAL_MACHINE, TEXT("SOFTWARE\\Microsoft\\Windows NT\\CurrentVersion"), TEXT("UBR"), UpdateBuildRevision))
					{
						UpdateBuildRevision = TEXT("UNKNOWN");
					}

					// Add the build number as displayed by the winver utility.
					OSVersionLabel += FString::Printf(TEXT(" [%u.%u.%u.%s]"), (uint32)OsVersionInfo.dwMajorVersion, (uint32)OsVersionInfo.dwMinorVersion, (uint32)OsVersionInfo.dwBuildNumber, *UpdateBuildRevision);
				}
				break;
			default:
				ErrorCode |= (int32)FWindowsOSVersionHelper::ERROR_UNKNOWNVERSION;
				break;
			}
			break;
		default:
			ErrorCode |= FWindowsOSVersionHelper::ERROR_UNKNOWNVERSION;
			break;
		}

		if(OsVersionInfo.dwMajorVersion >= 6)
		{
PRAGMA_DISABLE_CAST_FUNCTION_TYPE_MISMATCH_WARNINGS
			typedef BOOL( WINAPI *LPFN_GETPRODUCTINFO )(DWORD, DWORD, DWORD, DWORD, PDWORD);
			LPFN_GETPRODUCTINFO fnGetProductInfo = (LPFN_GETPRODUCTINFO)GetProcAddress( GetModuleHandle( TEXT( "kernel32.dll" ) ), "GetProductInfo" );
PRAGMA_ENABLE_CAST_FUNCTION_TYPE_MISMATCH_WARNINGS
			if( fnGetProductInfo != NULL )
			{
				DWORD Type;
				fnGetProductInfo( OsVersionInfo.dwMajorVersion, OsVersionInfo.dwMinorVersion, 0, 0, &Type );

				switch( Type )
				{
					case PRODUCT_ULTIMATE:
						OSSubVersionLabel = TEXT( "Ultimate Edition" );
						break;
					case PRODUCT_PROFESSIONAL:
						OSSubVersionLabel = TEXT( "Professional" );
						break;
					case PRODUCT_HOME_PREMIUM:
						OSSubVersionLabel = TEXT( "Home Premium Edition" );
						break;
					case PRODUCT_HOME_BASIC:
						OSSubVersionLabel = TEXT( "Home Basic Edition" );
						break;
					case PRODUCT_ENTERPRISE:
						OSSubVersionLabel = TEXT( "Enterprise Edition" );
						break;
					case PRODUCT_BUSINESS:
						OSSubVersionLabel = TEXT( "Business Edition" );
						break;
					case PRODUCT_STARTER:
						OSSubVersionLabel = TEXT( "Starter Edition" );
						break;
					case PRODUCT_CLUSTER_SERVER:
						OSSubVersionLabel = TEXT( "Cluster Server Edition" );
						break;
					case PRODUCT_DATACENTER_SERVER:
						OSSubVersionLabel = TEXT( "Datacenter Edition" );
						break;
					case PRODUCT_DATACENTER_SERVER_CORE:
						OSSubVersionLabel = TEXT( "Datacenter Edition (core installation)" );
						break;
					case PRODUCT_ENTERPRISE_SERVER:
						OSSubVersionLabel = TEXT( "Enterprise Edition" );
						break;
					case PRODUCT_ENTERPRISE_SERVER_CORE:
						OSSubVersionLabel = TEXT( "Enterprise Edition (core installation)" );
						break;
					case PRODUCT_ENTERPRISE_SERVER_IA64:
						OSSubVersionLabel = TEXT( "Enterprise Edition for Itanium-based Systems" );
						break;
					case PRODUCT_SMALLBUSINESS_SERVER:
						OSSubVersionLabel = TEXT( "Small Business Server" );
						break;
					case PRODUCT_SMALLBUSINESS_SERVER_PREMIUM:
						OSSubVersionLabel = TEXT( "Small Business Server Premium Edition" );
						break;
					case PRODUCT_STANDARD_SERVER:
						OSSubVersionLabel = TEXT( "Standard Edition" );
						break;
					case PRODUCT_STANDARD_SERVER_CORE:
						OSSubVersionLabel = TEXT( "Standard Edition (core installation)" );
						break;
					case PRODUCT_WEB_SERVER:
						OSSubVersionLabel = TEXT( "Web Server Edition" );
						break;
				}
			}
			else
			{
				OSSubVersionLabel = TEXT( "(type unknown)" );
				ErrorCode |= (int32)FWindowsOSVersionHelper::ERROR_GETPRODUCTINFO_FAILED;
			}
		}

#if 0
		// THIS BIT ADDS THE SERVICE PACK INFO TO THE EDITION STRING
		// Append service pack info
		if( OsVersionInfo.szCSDVersion[0] != 0 )
		{
			OSSubVersionLabel += FString::Printf( TEXT( " (%s)" ), OsVersionInfo.szCSDVersion );
		}
#else
		// THIS BIT USES SERVICE PACK INFO ONLY
		OSSubVersionLabel = OsVersionInfo.szCSDVersion;
#endif
	}
	else
	{
		ErrorCode |= FWindowsOSVersionHelper::ERROR_GETVERSIONEX_FAILED;
	}

	FCString::Strncpy(OutOSVersionLabel, *OSVersionLabel, OSVersionLabelLength);
	FCString::Strncpy(OutOSSubVersionLabel, *OSSubVersionLabel, OSSubVersionLabelLength);

	return ErrorCode;
}

int32 FWindowsOSVersionHelper::GetOSVersions( FString& OutOSVersionLabel, FString& OutOSSubVersionLabel )
{
	TCHAR OSVersionLabel[128];
	TCHAR OSSubVersionLabel[128];

	OSVersionLabel[0] = TEXT('\0');
	OSSubVersionLabel[0] = TEXT('\0');

	int32 Result = GetOSVersionsHelper( OSVersionLabel, UE_ARRAY_COUNT(OSVersionLabel), OSSubVersionLabel, UE_ARRAY_COUNT(OSSubVersionLabel) );

	OutOSVersionLabel = OSVersionLabel;
	OutOSSubVersionLabel = OSSubVersionLabel;

	return Result;
}

namespace
{
	bool GetOSVersionHelper(TCHAR* OutString, int32 Length)
	{
		int32 ErrorCode = (int32)FWindowsOSVersionHelper::SUCCEEDED;

		// Get system info
		SYSTEM_INFO SystemInfo;
		GetNativeSystemInfo(&SystemInfo);

		OSVERSIONINFOEX OsVersionInfo = { 0 };
		OsVersionInfo.dwOSVersionInfoSize = sizeof(OSVERSIONINFOEX);
	#ifdef __clang__
	#pragma clang diagnostic push
	#pragma clang diagnostic ignored "-Wdeprecated-declarations"
	#else
	#pragma warning(push)
	#pragma warning(disable : 4996) // 'function' was declared deprecated
	#endif
		CA_SUPPRESS(28159)
		if (GetVersionEx((LPOSVERSIONINFO)&OsVersionInfo))
	#ifdef __clang__
	#pragma clang diagnostic pop
	#else
	#pragma warning(pop)
	#endif
		{
			FCString::Snprintf(OutString, Length, TEXT("%d.%d.%d.%d.%d.64bit"), OsVersionInfo.dwMajorVersion, OsVersionInfo.dwMinorVersion, OsVersionInfo.dwBuildNumber, OsVersionInfo.wProductType, OsVersionInfo.wSuiteMask);
			return true;
		}
		return false;
	}

	struct FStorageDevice
	{
		FString SerialNumber;
		WIDECHAR Drive;
		FPlatformDriveStats Stats;

		FStorageDevice(FString&& InSerialNumber, WIDECHAR DriveLetter)
			: SerialNumber(MoveTemp(InSerialNumber))
			, Drive(DriveLetter)
			, Stats{ DriveLetter, 0, 0, EStorageDeviceType ::Unknown}
		{
			SerialNumber.TrimStartAndEndInline();
		}
	};

	// NOTE: This is populated async (if GCollectStorageInformationAsync)
	// Usage must be synchronized with WaitOnCollectStorageInformationAsync
	TArray<FStorageDevice> StorageDevices;

	static void LogStorageInformationWarning(HRESULT HRes,  const TCHAR* message)
	{
		IErrorInfo* ErrorInfo = nullptr;
		GetErrorInfo(0, &ErrorInfo);
		if (ErrorInfo)
		{
			BSTR ErrorMessage;
			ErrorInfo->GetDescription(&ErrorMessage);
			if (ErrorMessage && *ErrorMessage)
			{
				UE_LOGF(LogWindows, Log, "%ls [%ls]", message, ErrorMessage);
			}
			else
			{
				UE_LOGF(LogWindows, Log, "%ls [error code %x]", message, HRes);
			}
			::SysFreeString(ErrorMessage);
			ErrorInfo->Release();
		}
		else
		{
			UE_LOGF(LogWindows, Log, "%ls [error code %x]", message, HRes);
		}
	}
	
#if ALLOW_COLLECT_STORAGE_INFORMATION
	static void CollectStorageInformationImpl()
	{
		IWbemLocator* WbemLocator = nullptr;
		IWbemServices* WbemServices = nullptr;

		if (!FWindowsPlatformMisc::CoInitialize())
		{
			return;
		}
		HRESULT hres = CoInitializeSecurity(NULL, -1, NULL, NULL, RPC_C_AUTHN_LEVEL_DEFAULT, RPC_C_IMP_LEVEL_IMPERSONATE, NULL, EOAC_NONE, NULL);
		if (FAILED(hres))
		{
			LogStorageInformationWarning(hres, TEXT("Error initializing COM"));
			FWindowsPlatformMisc::CoUninitialize();
			return;
		}
		hres = CoCreateInstance(CLSID_WbemLocator, 0, CLSCTX_INPROC_SERVER, IID_IWbemLocator, (LPVOID*)&WbemLocator);
		if (FAILED(hres))
		{
			LogStorageInformationWarning(hres, TEXT("Error creating Wbem instance"));
			FWindowsPlatformMisc::CoUninitialize();
			return;
		}
		hres = WbemLocator->ConnectServer(_bstr_t(L"ROOT\\microsoft\\windows\\storage"), NULL, NULL, 0, NULL, 0, 0, &WbemServices);
		if (FAILED(hres))
		{
			LogStorageInformationWarning(hres, TEXT("Error connecting to storage service"));
			WbemLocator->Release();
			FWindowsPlatformMisc::CoUninitialize();
			return;
		}
		hres = CoSetProxyBlanket(
			WbemServices,
			RPC_C_AUTHN_WINNT,
			RPC_C_AUTHZ_NONE,
			NULL,
			RPC_C_AUTHN_LEVEL_CALL,
			RPC_C_IMP_LEVEL_IMPERSONATE,
			NULL,
			EOAC_NONE
		);
		if (FAILED(hres))
		{
			LogStorageInformationWarning(hres, TEXT("Error setting authentication information"));
			WbemServices->Release();
			WbemLocator->Release();
			FWindowsPlatformMisc::CoUninitialize();
			return;
		}


		IEnumWbemClassObject* StorageEnumerator = nullptr;
		hres = WbemServices->ExecQuery(
			bstr_t("WQL"),
			bstr_t("SELECT * FROM MSFT_DiskToPartition"),
			WBEM_FLAG_FORWARD_ONLY,
			NULL,
			&StorageEnumerator);

		if (FAILED(hres))
		{
			LogStorageInformationWarning(hres, TEXT("Error listing partitions"));
			WbemServices->Release();
			WbemLocator->Release();
			FWindowsPlatformMisc::CoUninitialize();
			return;
		}

		// Enumerate all partitions, and store disk information
		while (StorageEnumerator)
		{
			ULONG Returned;
			IWbemClassObject* StorageWbemObject[10]{};
			hres = StorageEnumerator->Next(WBEM_INFINITE, 10, StorageWbemObject, &Returned);
			if (FAILED(hres))
			{
				LogStorageInformationWarning(hres, TEXT("Error iterating over partitions"));
				break;
			}
			else if (Returned == 0)
			{
				break;
			}
			for (ULONG i = 0; i < Returned; ++i)
			{
				VARIANT Disk;
				VARIANT Partition;

				StorageWbemObject[i]->Get(L"Disk", 0, &Disk, NULL, NULL);
				StorageWbemObject[i]->Get(L"Partition", 0, &Partition, NULL, NULL);

				IWbemClassObject* PartitionObject = nullptr;
				IWbemClassObject* DiskObject = nullptr;
				hres = WbemServices->GetObject(
					Partition.bstrVal,
					WBEM_FLAG_RETURN_WBEM_COMPLETE,
					NULL,
					&PartitionObject,
					NULL);
				if (SUCCEEDED(hres))
				{
					hres = WbemServices->GetObject(
						Disk.bstrVal,
						WBEM_FLAG_RETURN_WBEM_COMPLETE,
						NULL,
						&DiskObject,
						NULL);
					if (SUCCEEDED(hres))
					{
						VARIANT Drive;
						hres = PartitionObject->Get(L"DriveLetter", 0, &Drive, NULL, NULL);
						if (SUCCEEDED(hres))
						{
							if (Drive.uiVal != 0)
							{
								VARIANT SerialNumber;
								hres = DiskObject->Get(L"SerialNumber", 0, &SerialNumber, NULL, NULL);
								if (SUCCEEDED(hres))
								{
									StorageDevices.Emplace(SerialNumber.bstrVal, Drive.uiVal);
									VariantClear(&SerialNumber);
								}
								else
								{
									LogStorageInformationWarning(hres, TEXT("Error retrieving serial number"));
								}
								VariantClear(&Drive);
							}
						}
						else
						{
							LogStorageInformationWarning(hres, TEXT("Error retrieving drive letter"));
						}
						DiskObject->Release();
					}
					else
					{
						LogStorageInformationWarning(hres, TEXT("Error retrieving disk information"));
					}
					PartitionObject->Release();
				}
				else
				{
					LogStorageInformationWarning(hres, TEXT("Error retrieving partition information"));
				}

				VariantClear(&Disk);
				VariantClear(&Partition);
				StorageWbemObject[i]->Release();
			}
		}

		StorageEnumerator->Release();
		hres = WbemServices->ExecQuery(
			bstr_t("WQL"),
			bstr_t("SELECT * FROM MSFT_PhysicalDisk"),
			WBEM_FLAG_FORWARD_ONLY,
			NULL,
			&StorageEnumerator);

		if (FAILED(hres))
		{
			LogStorageInformationWarning(hres, TEXT("Error when querying physical disks"));
			WbemServices->Release();
			WbemLocator->Release();
			FWindowsPlatformMisc::CoUninitialize();
			return;
		}

		while (StorageEnumerator)
		{
			ULONG Returned;
			IWbemClassObject* StorageWbemObject = nullptr;
			hres = StorageEnumerator->Next(WBEM_INFINITE, 1, &StorageWbemObject, &Returned);
			if (FAILED(hres))
			{
				LogStorageInformationWarning(hres, TEXT("Error when iterating over physical disks"));
				break;
			}
			else if (Returned == 0)
			{
				break;
			}

			// see https://learn.microsoft.com/en-us/windows-hardware/drivers/storage/msft-physicaldisk for other properties
			VARIANT SerialNumber;
			VARIANT MediaType;
			VARIANT BusType;
			VARIANT SpindleSpeed;

			StorageWbemObject->Get(L"SerialNumber", 0, &SerialNumber, NULL, NULL);
			StorageWbemObject->Get(L"MediaType", 0, &MediaType, NULL, NULL);
			StorageWbemObject->Get(L"BusType", 0, &BusType, NULL, NULL);
			StorageWbemObject->Get(L"SpindleSpeed", 0, &SpindleSpeed, NULL, NULL);

			FString OriginalSerial(SerialNumber.bstrVal);
			// Use a stringview as ::TrimStartAndEndInline would need to move data around if there is whitespace at the start of the string
			FStringView Serial = FStringView(OriginalSerial).TrimStartAndEnd();

			for (FStorageDevice& StorageDevice : StorageDevices)
			{
				if (StorageDevice.SerialNumber == Serial)
				{
					if (MediaType.uiVal == 3) // HDD
					{
						StorageDevice.Stats.DriveType = EStorageDeviceType::HDD;
					}
					else if (MediaType.uiVal == 4) // SSD
					{
						if (BusType.uiVal == 17) // NVMe
						{
							StorageDevice.Stats.DriveType = EStorageDeviceType::NVMe;
						}
						else if (SpindleSpeed.uintVal != 0)
						{
							StorageDevice.Stats.DriveType = EStorageDeviceType::Hybrid;
						}
						else
						{
							StorageDevice.Stats.DriveType = EStorageDeviceType::SSD;
						}
					}
					else
					{
						StorageDevice.Stats.DriveType = EStorageDeviceType::Other;
					}
				}
			}

			VariantClear(&BusType);
			VariantClear(&MediaType);
			VariantClear(&SerialNumber);
			StorageWbemObject->Release();
		}

		StorageEnumerator->Release();
		WbemServices->Release();
		WbemLocator->Release();
		FWindowsPlatformMisc::CoUninitialize();
		FWindowsPlatformMisc::UpdateDriveFreeSpace();
	}

	struct FCollectStorageInformationAsyncThread
	{
		HANDLE HThread;
		DWORD HThreadId;

		FCollectStorageInformationAsyncThread()
			: HThread(NULL)
			, HThreadId(0)
		{
		}

		~FCollectStorageInformationAsyncThread()
		{
			if (HThread)
			{
				if (::WaitForSingleObject(HThread, 1000) == WAIT_OBJECT_0)
				{
					::CloseHandle(HThread);
				}
				HThread = NULL;
			}
		}
	};
	static FCollectStorageInformationAsyncThread GCollectStorageInformationAsyncThread;
	
	static DWORD WINAPI CollectStorageInformationAsyncProc(LPVOID Arg)
	{
		CollectStorageInformationImpl();
		return 0;
	}

	static void CollectStorageInformationAsync()
	{
		GCollectStorageInformationAsyncThread.HThread = ::CreateThread(NULL, 0, CollectStorageInformationAsyncProc, NULL, 0, &GCollectStorageInformationAsyncThread.HThreadId);
		if (GCollectStorageInformationAsyncThread.HThread == NULL)
		{
			UE_LOGF(LogWindows, Error, "Unable to create thread for CollectStorageInformationAsync, running synchronous...");
			CollectStorageInformationImpl();
		}
	}
#endif // ALLOW_COLLECT_STORAGE_INFORMATION

	static void WaitOnCollectStorageInformationAsync()
	{
#if ALLOW_COLLECT_STORAGE_INFORMATION
		if (GCollectStorageInformationAsync && ::GetCurrentThreadId() != GCollectStorageInformationAsyncThread.HThreadId)
		{
			::WaitForSingleObject(GCollectStorageInformationAsyncThread.HThread, INFINITE);
		}
#endif
	}

	static void CollectStorageInformation()
	{
#if ALLOW_COLLECT_STORAGE_INFORMATION
		if (GCollectStorageInformationAsync)
		{
			CollectStorageInformationAsync();
		}
		else
		{
			CollectStorageInformationImpl();
		}
#endif
	}
}

const TCHAR* LexToString(EStorageDeviceType StorageType)
{
	switch (StorageType)
	{
	case EStorageDeviceType::Other:
		return TEXT("Other");
	case EStorageDeviceType::HDD:
		return TEXT("HDD");
	case EStorageDeviceType::SSD:
		return TEXT("SSD");
	case EStorageDeviceType::NVMe:
		return TEXT("NVMe");
	case EStorageDeviceType::Hybrid:
		return TEXT("Hybrid");
	case EStorageDeviceType::Unknown:
		[[fallthrough]];
	default:
		return TEXT("Unknown");
	}
}

const TCHAR* LexToString(ETDRLevel TDRLevel)
{
	switch (TDRLevel)
	{
	case ETDRLevel::Disabled:
		return TEXT("Disabled");
	case ETDRLevel::BugCheck:
		return TEXT("BugCheck");
	case ETDRLevel::RecoverVGA:
		return TEXT("RecoverVGA");
	case ETDRLevel::Recover:
		return TEXT("Recover");
	case ETDRLevel::Invalid:
		[[fallthrough]];
	default:
		return TEXT("Invalid");
	}
}

#include "Windows/HideWindowsPlatformTypes.h"

/** 
 * Whether support for integrating into the firewall is there
 */
#define WITH_FIREWALL_SUPPORT	0

extern "C"
{
	CORE_API Windows::HINSTANCE hInstance = NULL;
}


/** Original C- Runtime pure virtual call handler that is being called in the (highly likely) case of a double fault */
_purecall_handler DefaultPureCallHandler;

/**
* Our own pure virtual function call handler, set by appPlatformPreInit. Falls back
* to using the default C- Runtime handler in case of double faulting.
*/
static void PureCallHandler()
{
	static bool bHasAlreadyBeenCalled = false;
	UE_DEBUG_BREAK();
	if( bHasAlreadyBeenCalled )
	{
		// Call system handler if we're double faulting.
		if( DefaultPureCallHandler )
		{
			DefaultPureCallHandler();
		}
	}
	else
	{
		bHasAlreadyBeenCalled = true;
		if( GIsRunning )
		{
			FMessageDialog::Open( EAppMsgType::Ok, NSLOCTEXT("Core", "PureVirtualFunctionCalledWhileRunningApp", "Pure virtual function being called while application was running (GIsRunning == 1).") );
		}
		UE_LOGF(LogWindows, Fatal,"Pure virtual function being called" );
	}
}

#if ENABLE_PGO_PROFILE && !defined(__clang__) && !defined(__INTEL_LLVM_COMPILER)
void PGO_WriteFile()
{
	// NB. Using pgosweep.exe means the PGC file will be writable as soon as the title exits & we can control where it is written.
	// Not using PgoAutoSweep because a) it calls MessageBox() when it encounters an error which would break unattended automation,
	// and b) it only takes a file name fragment not a full path, so it would be necessary to sweep, find the file and then move it where we want it.

	static uint32 FileCounter = 0;

	// Get the current running process's full path
	TCHAR ExeFilePath[MAX_PATH + 1];
	GetModuleFileNameW(NULL, ExeFilePath, MAX_PATH + 1);
	FString ExeFileName = FPaths::GetCleanFilename(ExeFilePath);
	FString ExeFolder = FPaths::GetPath(ExeFilePath);
	FString ExeFileNameWithoutExtension = FPaths::GetBaseFilename(ExeFilePath);

	// Get PGC output directory, defaulting to the exe location but can sweep to the project saved dir so Gauntlet can collect it
	bool bSweepToSaveDir = FParse::Param(FCommandLine::Get(), TEXT("PGOSweepToSaveDir"));
	FString OutputDirectory = bSweepToSaveDir  ?  FPaths::Combine(FPaths::ProjectSavedDir(), TEXT("PGO"))  :  ExeFolder;

	// Find next unique PGC file name
	FString OutputFilePath;
	do
	{
		FString PGCFileName = FString::Printf(TEXT("%s!%d.pgc"), *ExeFileNameWithoutExtension, ++FileCounter);
		OutputFilePath = FPaths::Combine(OutputDirectory, PGCFileName).Replace(TEXT("/"), TEXT("\\"));
	} while (GetFileAttributesW(*OutputFilePath) != INVALID_FILE_ATTRIBUTES);

	// Launch PGOSweep & wait for it to finish
#if PLATFORM_WINDOWS_ARM64EC
	FString PGOSweepPath = FPaths::Combine(ExeFolder, TEXT("arm64ec"), TEXT("pgosweep.exe"));
#elif PLATFORM_CPU_ARM_FAMILY
	FString PGOSweepPath = FPaths::Combine(ExeFolder, TEXT("arm64"), TEXT("pgosweep.exe"));
#else
	FString PGOSweepPath = FPaths::Combine(ExeFolder, TEXT("pgosweep.exe"));
#endif
	
	FString CommandLine = FString::Printf(TEXT("/pid:%d \"%s\" \"%s\""), ::GetCurrentProcessId(), ExeFilePath, *OutputFilePath);

	const bool bLaunchDetached = true;
	const bool bLaunchHidden = true;
	const bool bLaunchReallyHidden = bLaunchHidden;
	FProcHandle ProcHandle = FPlatformProcess::CreateProc(*PGOSweepPath, *CommandLine, bLaunchDetached, bLaunchHidden, bLaunchReallyHidden, nullptr, 0, nullptr, nullptr, nullptr);

	if (ProcHandle.IsValid())
	{
		FPlatformProcess::WaitForProc(ProcHandle);
		int32 ExitCode = 0;
		if (FPlatformProcess::GetProcReturnCode(ProcHandle, &ExitCode))
		{
			UE_LOGF(LogWindows, Log, "pgosweep.exe exit code %d", ExitCode);
		}
		FPlatformProcess::CloseProc(ProcHandle);
	}
	else
	{
		UE_LOGF(LogWindows, Error, "Failed to launch pgosweep.exe");
	}
}
#endif //ENABLE_PGO_PROFILE && !__clang__ && !__INTEL_LLVM_COMPILER


/*-----------------------------------------------------------------------------
	SHA-1 functions.
-----------------------------------------------------------------------------*/

/**
* Get the hash values out of the executable hash section
*
* NOTE: hash keys are stored in the executable, you will need to put a line like the
*		 following into your PCLaunch.rc settings:
*			ID_HASHFILE RCDATA "../../../../GameName/Build/Hashes.sha"
*
*		 Then, use the -sha option to the cooker (must be from commandline, not
*       frontend) to generate the hashes for .ini, loc, startup packages, and .usf shader files
*
*		 You probably will want to make and checkin an empty file called Hashses.sha
*		 into your source control to avoid linker warnings. Then for testing or final
*		 final build ONLY, use the -sha command and relink your executable to put the
*		 hashes for the current files into the executable.
*/
static void InitSHAHashes()
{
	uint32 SectionSize = 0;
	void* SectionData = NULL;
	// find the resource for the file hash in the exe by ID
	HRSRC HashFileFindResH = FindResource(NULL,MAKEINTRESOURCE(ID_HASHFILE),RT_RCDATA);
	if( HashFileFindResH )
	{
		// load it
		HGLOBAL HashFileLoadResH = LoadResource(NULL,HashFileFindResH);
		if( !HashFileLoadResH )
		{
			FMessageDialog::ShowLastError();
		}
		else
		{
			// get size
			SectionSize = SizeofResource(NULL,HashFileFindResH);
			// get the data. no need to unlock it
			SectionData = (uint8*)LockResource(HashFileLoadResH);
		}
	}

	// there may be a dummy byte for platforms that can't handle empty files for linking
	if (SectionSize <= 1)
	{
		return;
	}

	// look for the hash section
	if( SectionData )
	{
		FSHA1::InitializeFileHashesFromBuffer((uint8*)SectionData, SectionSize);
	}
}

/**
 *	Sets process memory limit using the job object, may fail under some situation like when Program Compatibility Assistant is enabled.
 *	Debugging purpose only.
 */
static void SetProcessMemoryLimit( SIZE_T ProcessMemoryLimitMB )
{
	HANDLE JobObject = ::CreateJobObject(nullptr, TEXT("UE4-JobObject"));
	check(JobObject);
	JOBOBJECT_EXTENDED_LIMIT_INFORMATION JobLimitInfo;
	FMemory::Memzero( JobLimitInfo );
	JobLimitInfo.ProcessMemoryLimit = 1024*1024*ProcessMemoryLimitMB;
	JobLimitInfo.BasicLimitInformation.LimitFlags = JOB_OBJECT_LIMIT_PROCESS_MEMORY;
	const BOOL bSetJob = ::SetInformationJobObject(JobObject,JobObjectExtendedLimitInformation,&JobLimitInfo,sizeof(JobLimitInfo));

	const BOOL bAssign = ::AssignProcessToJobObject(JobObject, GetCurrentProcess());
}

void FWindowsPlatformMisc::PlatformPreInit()
{
	//SetProcessMemoryLimit( 92 );

	FGenericPlatformMisc::PlatformPreInit();

#if WITH_APPLICATION_CORE && WITH_WINDOWS_CONFIGRULES
	{
		SCOPED_BOOT_TIMING("FWindowsConfigRules::Init");
		FWindowsConfigRules::Init();
	}
	{
		SCOPED_BOOT_TIMING("FWindowsConfigRules::Parse");
		FWindowsConfigRules::SetStandardPredefinedVars();
		FWindowsConfigRules::Parse();
	}
#endif // WITH_APPLICATION_CORE && WITH_WINDOWS_CONFIGRULES

	FThreadHeartBeat::Get().GetOnThreadStuck().BindStatic(&FGenericCrashContext::OnThreadStuck);
	FThreadHeartBeat::Get().GetOnThreadUnstuck().BindStatic(&FGenericCrashContext::OnThreadUnstuck);

	// Load the bundled version of dbghelp.dll if necessary
#if USE_BUNDLED_DBGHELP
	// Loading newer versions of DbgHelp fails on Windows 7 since it is no longer supported.
	if (IsWindows8OrGreater())
	{
		// Try to load a bundled copy of dbghelp.dll. A bug with Windows 10 version 1709 
#if PLATFORM_CPU_ARM_FAMILY && !PLATFORM_WINDOWS_ARM64EC
		FString DbgHlpPath = FPaths::EngineDir() / TEXT("Binaries/ThirdParty/DbgHelp/arm64/dbghelp.dll");
#else
		FString DbgHlpPath = FPaths::EngineDir() / TEXT("Binaries/ThirdParty/DbgHelp/dbghelp.dll");
#endif
		FPlatformProcess::GetDllHandle(*DbgHlpPath);
	}
#endif

	// Use our own handler for pure virtuals being called.
	DefaultPureCallHandler = _set_purecall_handler( PureCallHandler );

	const int32 MinResolution[] = {640,480};
	if ( ::GetSystemMetrics(SM_CXSCREEN) < MinResolution[0] || ::GetSystemMetrics(SM_CYSCREEN) < MinResolution[1] )
	{
		FMessageDialog::Open( EAppMsgType::Ok, NSLOCTEXT("Launch", "Error_ResolutionTooLow", "The current resolution is too low to run this game.") );
		FPlatformMisc::RequestExit( false, TEXT("FWindowsPlatformMisc::PlatformPreInit.ResolutionTooLow"));
	}

	// initialize the file SHA hash mapping
	InitSHAHashes();

#if !PLATFORM_CPU_ARM_FAMILY
	// Check for SSE42 or better. This is now minspec and there is a high likelihood
	// of crashing on an invalid instruction on unsupported processors as we use these
	// instructions now.
	if (CheckFeatureBit_X86(ECPUFeatureBits_X86::SSE42) == false)
	{
		FMessageDialog::Open(EAppMsgType::Ok, NSLOCTEXT("Launch", "Error_CPUNotSupported", "This CPU does not support a required feature (SSE4.2)."));
		FPlatformMisc::RequestExit(false, TEXT("FWindowsPlatformMisc::PlatformPreInit.CPUNotSupported"));
	}
#endif
}

void FWindowsPlatformMisc::PlatformInit()
{
	FGenericPlatformMisc::LogNameEventStatsInit();

#if defined(_MSC_VER) && _MSC_VER == 1800
	// Work around bug in the VS 2013 math libraries in 64bit on certain windows versions. http://connect.microsoft.com/VisualStudio/feedback/details/811093 has details, remove this when runtime libraries are fixed
	_set_FMA3_enable(0);
#endif

	// Set granularity of sleep and such to 1 ms.
	timeBeginPeriod( 1 );

	// Identity.
	UE_LOGF(LogInit, Log, "Computer: %ls", FPlatformProcess::ComputerName() );
	UE_LOGF(LogInit, Log, "User: %ls", FPlatformProcess::UserName() );

	// Get CPU info.
	const FPlatformMemoryConstants& MemoryConstants = FPlatformMemory::GetConstants();
	UE_LOG(LogInit, Log, TEXT("CPU Page size=%" SIZE_T_FMT ", Cores=%i"), MemoryConstants.PageSize, FPlatformMisc::NumberOfCores() );

	// Timer resolution.
	UE_LOGF(LogInit, Log, "High frequency timer resolution =%f MHz", 0.000001 / FPlatformTime::GetSecondsPerCycle() );

	// If the Device has been identified as a Handheld device
	UE_LOGF(LogInit, Log, "Gaming Handheld Device: %d", FPlatformMisc::IsDeviceGamingHandheld() ? 1 : 0);

	// Register on the game thread.
	FWindowsPlatformStackWalk::RegisterOnModulesChanged();

	// Collecting storage information using dcom services.
	CollectStorageInformation();

	// Start listening for network connectivity changes via NLM events.
	StartNetworkConnectionMonitoring();
}

void FWindowsPlatformMisc::PlatformTearDown()
{
	// StopNetworkConnectionMonitoring is now called via FCoreDelegates::OnEnginePreExit as this function is called too late in the exit sequence
	FPlatformProcess::CeaseBeingFirstInstance();
}

/**
 * Handler called for console events like closure, CTRL-C, ...
 *
 * @param Type Ctrl-C, Ctrl-Break, Close Console Log Window, Logoff, or Shutdown
 */
static BOOL WINAPI ConsoleCtrlHandler(DWORD CtrlType)
{
	// Broadcast the termination the first time through.
	bool IsRequestingExit = IsEngineExitRequested();
	static bool AppTermDelegateBroadcast = false;
	if (!AppTermDelegateBroadcast)
	{
		RequestEngineExit(TEXT("ConsoleCtrl RequestExit"));

		FCoreDelegates::GetApplicationWillTerminateDelegate().Broadcast();

		AppTermDelegateBroadcast = true;
	}

	// Only "two-step Ctrl-C" if the termination event is Ctrl-C and the process
	// is considered interactive. Hard-terminate on all other cases.
	if (CtrlType != CTRL_C_EVENT || FApp::IsUnattended())
	{
		IsRequestingExit = true;
	}

	if (!IsRequestingExit)
	{
		UE_LOGF(LogCore, Warning, "*** INTERRUPTED *** : SHUTTING DOWN");
		UE_LOGF(LogCore, Warning, "*** INTERRUPTED *** : CTRL-C TO FORCE QUIT");
	}

	// make sure as much data is written to disk as possible
	if (GLog)
	{
		GLog->Flush();
	}
	if (GWarn)
	{
		GWarn->Flush();
	}
	if (GError)
	{
		GError->Flush();
	}

	if (!IsRequestingExit)
	{
		// We'll two-step Ctrl-C events to give processes like servers time to
		// correctly terminate. Note the IsEngineExitRequested() is true now.
		return true;
	}

	// There's no guarantee that the process is paying attention to IsEngineExitRequested()
	// (e.g. some long-running commandlets). Using that for shutdown is therefore not
	// reliable. Deferring to the default CtrlHandler is also no good as that calls
	// ExitProcess() which will terminate all threads and detach all DLLS. This can
	// result in deadlocks and/or asserts as each DLL's atexit() is processed. So
	// let's hard terminate the process. 0xc000013a is what Windows' default handler
	// would normally pass to ExitProcess() and how ExitProc() terminates threads.
	TerminateProcess(GetCurrentProcess(), 0xc000013au);
	return false;
}

void FWindowsPlatformMisc::SetGracefulTerminationHandler()
{
	if (GetConsoleWindow() == nullptr)
	{
		return;
	}

	// Set console control handler so we can exit if requested.
	SetConsoleCtrlHandler(ConsoleCtrlHandler, true);

#if !UE_BUILD_SHIPPING && PLATFORM_CPU_X86_FAMILY
	// There are many places that can register a Ctrl-C handler, some of which
	// we are not in control of (third party Perforce libraries for example). This
	// can result in Ctrl-C doing nothing, or an abrupt call to ExitProcess() which
	// will cause asserts and/or deadlocks. So we're going to apply a patch so no
	// one else can register a handler.
	auto* SetCtrlCProc = (uint8*)SetConsoleCtrlHandler;
	if (SetCtrlCProc[0] == 0xff && SetCtrlCProc[1] == 0x25)
	{
		// Follow a possible "jmp [eip] + disp32" instruction to get to the actual
		// implementation of the SetConsoleCtrlHandler function.
		SetCtrlCProc = *(uint8**)(SetCtrlCProc + 6 + *(uint32*)(SetCtrlCProc + 2));
	}

	// Patch the start of the SetConsoleCtrlHandler function.
	uint8 Patch[] = {
		0xb8, 0x01, 0x00, 0x00, 0x00,	// mov eax, 1
		0xc3							// ret
	};
	DWORD PrevProtection;
	if (VirtualProtect(SetCtrlCProc, sizeof(Patch), PAGE_EXECUTE_READWRITE, &PrevProtection))
	{
		memcpy(SetCtrlCProc, Patch, sizeof(Patch));
		VirtualProtect(SetCtrlCProc, sizeof(Patch), PrevProtection, &PrevProtection);
		FlushInstructionCache(GetCurrentProcess(), nullptr, 0);
	}
#endif // !UE_BUILD_SHIPPING && PLATFORM_CPU_X86_FAMILY
}

void FWindowsPlatformMisc::CallGracefulTerminationHandler()
{
	ConsoleCtrlHandler(CTRL_CLOSE_EVENT);
}

static ECrashHandlingType GCrashHandlingType; /* = ECrashHandlingType::Default */

ECrashHandlingType FWindowsPlatformMisc::GetCrashHandlingType()
{
	return GCrashHandlingType;
}

ECrashHandlingType FWindowsPlatformMisc::SetCrashHandlingType(ECrashHandlingType Type)
{
	GCrashHandlingType = Type;
	return GCrashHandlingType;
}

int32 FWindowsPlatformMisc::GetMaxPathLength()
{
	struct FLongPathsEnabled
	{
		bool bValue;

		FLongPathsEnabled()
		{
			HMODULE Handle = GetModuleHandle(TEXT("ntdll.dll"));
			if (Handle == NULL)
			{
				bValue = false;
			}
			else
			{
				typedef BOOLEAN(NTAPI *RtlAreLongPathsEnabledFunc)();
				RtlAreLongPathsEnabledFunc RtlAreLongPathsEnabled = (RtlAreLongPathsEnabledFunc)(void*)GetProcAddress(Handle, "RtlAreLongPathsEnabled");
				if (RtlAreLongPathsEnabled != NULL)
				{
					bValue = RtlAreLongPathsEnabled();
				}
				else
				{
					// Long paths are always supported under Wine
					bValue = FWindowsPlatformMisc::IsWine();
				}
			}
		}
	};

	static FLongPathsEnabled LongPathsEnabled;
	return LongPathsEnabled.bValue? 32767 : MAX_PATH;
}

int32 FWindowsPlatformMisc::GetExternalAppMaxPathLength()
{
	// On Windows, return the minimum path limit regardless of long-path configuration, since external processes could still be not long-path aware.
	return MAX_PATH;
}

FString FWindowsPlatformMisc::GetEnvironmentVariable(const TCHAR* VariableName)
{
	// Allocate the data for the string. Loop in case the variable happens to change while running, or the buffer isn't large enough.
	FString Buffer;
	for(uint32 Length = 128;;)
	{
		TArray<TCHAR, FString::AllocatorType>& CharArray = Buffer.GetCharArray();
		CharArray.SetNumUninitialized(Length);

		Length = ::GetEnvironmentVariableW(VariableName, CharArray.GetData(), CharArray.Num());
		if (Length == 0)
		{
			Buffer.Reset();
			break;
		}
		else if (Length < (uint32)CharArray.Num())
		{
			CharArray.SetNum(Length + 1);
			break;
		}
	}
	return Buffer;
}

void FWindowsPlatformMisc::SetEnvironmentVar(const TCHAR* VariableName, const TCHAR* Value)
{
	if (Value != nullptr && Value[0] == TEXT('\0'))
	{
		Value = nullptr;
	}
	
	uint32 Error = ::SetEnvironmentVariable(VariableName, Value);
	if (Error == 0)
	{
		UE_LOGF(LogWindows, Warning, "Failed to set EnvironmentVariable: %ls to : %ls", VariableName, Value);
	}
}

TArray<uint8> FWindowsPlatformMisc::GetMacAddress()
{
	TArray<uint8> Result;
	IP_ADAPTER_INFO IpAddresses[16];
	ULONG OutBufferLength = sizeof(IP_ADAPTER_INFO) * 16;
	// Read the adapters
	uint32 RetVal = GetAdaptersInfo(IpAddresses,&OutBufferLength);
	if (RetVal == NO_ERROR)
	{
		PIP_ADAPTER_INFO AdapterList = IpAddresses;
		// Walk the set of addresses copying each one
		while (AdapterList)
		{
			// If there is an address to read
			if (AdapterList->AddressLength > 0)
			{
				// Copy the data and say we did
				Result.AddZeroed(AdapterList->AddressLength);
				FMemory::Memcpy(Result.GetData(),AdapterList->Address,AdapterList->AddressLength);
				break;
			}
			AdapterList = AdapterList->Next;
		}
	}
	return Result;
}

/**
 * We need to see if we are doing AutomatedPerfTesting and we are -unattended if we are then we have
 * crashed in some terrible way and we need to make certain we can kill -9 the devenv process /
 * vsjitdebugger.exe and any other processes that are still running
 */
static void HardKillIfAutomatedTesting()
{
	// so here 
	int32 FromCommandLine = 0;
	FParse::Value( FCommandLine::Get(), TEXT("AutomatedPerfTesting="), FromCommandLine );
	if(( FApp::IsUnattended() == true ) && ( FromCommandLine != 0 ) && ( FParse::Param(FCommandLine::Get(), TEXT("KillAllPopUpBlockingWindows")) == true ))
	{

		UE_LOGF(LogWindows, Warning, "Attempting to run KillAllPopUpBlockingWindows");

		// .bat files never seem to launch correctly with FPlatformProcess::CreateProc so we just use the FPlatformProcess::LaunchURL which will call ShellExecute
		// we don't really care about the return code in this case 
		FPlatformProcess::LaunchURL( TEXT("KillAllPopUpBlockingWindows.bat"), NULL, NULL );
	}
}


void FWindowsPlatformMisc::SubmitErrorReport( const TCHAR* InErrorHist, EErrorReportMode::Type InMode )
{
	if ((!FPlatformMisc::IsDebuggerPresent() || GAlwaysReportCrash) && !FParse::Param(FCommandLine::Get(), TEXT("CrashForUAT")))
	{
		HardKillIfAutomatedTesting();
	}
}

#if !UE_BUILD_SHIPPING
bool FWindowsPlatformMisc::IsDebuggerPresent()
{
	return !GIgnoreDebugger && !!::IsDebuggerPresent();
}

EProcessDiagnosticFlags FWindowsPlatformMisc::GetProcessDiagnostics()
{
	static EProcessDiagnosticFlags FoundDiagnostics = []() -> EProcessDiagnosticFlags
	{
		EProcessDiagnosticFlags Result = FGenericPlatformMisc::GetProcessDiagnostics();

		TCHAR* ImageFileName = PathFindFileName(FPlatformProcess::ExecutablePath());
		FString ImageFileSubkey = FString::Printf(TEXT("SOFTWARE\\Microsoft\\Windows NT\\CurrentVersion\\Image File Execution Options\\%s"), ImageFileName);

		// via https://docs.microsoft.com/en-us/windows-hardware/drivers/debugger/gflags-flag-table
		constexpr uint32 MemorySanitizerMask =
			0x00000010| // FLG_HEAP_ENABLE_TAIL_CHECK
			0x00000020| // FLG_HEAP_ENABLE_FREE_CHECK
			0x00000080| // FLG_HEAP_VALIDATE_ALL
			0x00000100| // FLG_APPLICATION_VERIFIER
			0x00000800| // FLG_HEAP_ENABLE_TAGGING
			0x00008000| // FLG_HEAP_ENABLE_TAG_BY_DLL
			0x00200000| // FLG_HEAP_DISABLE_COALESCING
			0x02000000; // FLG_HEAP_PAGE_ALLOCS

		DWORD Data, DataCount = sizeof(Data);
		if (ERROR_SUCCESS == RegGetValue(HKEY_LOCAL_MACHINE, *ImageFileSubkey, TEXT("GlobalFlag"), RRF_RT_REG_DWORD, nullptr, &Data, &DataCount))
		{
			if (MemorySanitizerMask & Data)
			{
				Result |= EProcessDiagnosticFlags::MemorySanitizer;
			}
		}

		return Result;
	}();

	return FoundDiagnostics;
}
#endif //!UE_BUILD_SHIPPING

void FWindowsPlatformMisc::AsymmetricThreadFenceHeavy()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPlatformMisc::AsymmetricThreadFenceHeavy);

	// The function generates an interprocessor interrupt (IPI) to all processors that are part of the current process affinity.
	// It guarantees the visibility of write operations performed on one processor to the other processors.
	FlushProcessWriteBuffers();
}

bool FWindowsPlatformMisc::IsRemoteSession()
{
	return ::GetSystemMetrics(SM_REMOTESESSION) != 0;
}

void FWindowsPlatformMisc::SetUTF8Output()
{
	// Unicode support requires the UTF-8 code page and the use of printf.
	// Note that this impacts stdout whether or not a console is attached.
	SetConsoleOutputCP(CP_UTF8);
}

void FWindowsPlatformMisc::LocalPrint(const TCHAR* Message)
{
#if USE_DEBUG_LOGGING
	// Use SRWLOCK directly until we have a trivially destructible FSharedMutex.
	static SRWLOCK Lock = SRWLOCK_INIT;

	// OutputDebugString has no documented maximum length but the OUTPUT_DEBUG_STRING_INFO
	// type that receives debug strings in debuggers has a WORD count of bytes including
	// the null terminator. Break up the string into parts if it exceeds this limit to
	// avoid truncation in debuggers that do not scan for the null terminator.
	constexpr int32 MaxOutputDebugStringLength = WORD_MAX / sizeof(TCHAR) - 1;
	if (const int32 Len = FCString::Strlen(Message); LIKELY(Len <= MaxOutputDebugStringLength))
	{
		AcquireSRWLockShared(&Lock);
		OutputDebugString(Message);
		ReleaseSRWLockShared(&Lock);
	}
	else
	{
		AcquireSRWLockExclusive(&Lock);
		FStringView MessageView(Message, Len);
		for (FString Part; !MessageView.IsEmpty(); MessageView.RightChopInline(MaxOutputDebugStringLength))
		{
			Part = MessageView.Left(MaxOutputDebugStringLength);
			OutputDebugString(*Part);
		}
		ReleaseSRWLockExclusive(&Lock);
	}
#endif
}

void FWindowsPlatformMisc::RequestExit( bool Force, const TCHAR* CallSite )
{
	UE_LOGF(LogWindows, Log,  "FPlatformMisc::RequestExit(%i, %ls)",
		Force, CallSite ? CallSite : TEXT("<NoCallSiteInfo>"));

	// Legacy behavior that now calls through to RequestExitWithStatus
	if( Force )
	{
		RequestExitWithStatus(Force, GIsCriticalError ? 3 : 0, CallSite);
	}
	else
	{
		RequestExitWithStatus(false, 0, CallSite);
	}
}

void FWindowsPlatformMisc::RequestExitWithStatus(bool Force, uint8 ReturnCode, const TCHAR* CallSite)
{
	UE_LOGF(LogWindows, Log, "FPlatformMisc::RequestExitWithStatus(%i, %i, %ls)", Force, ReturnCode,
		CallSite ? CallSite : TEXT("<NoCallSiteInfo>"));

#if ENABLE_PGO_PROFILE && !defined(__clang__) && !defined(__INTEL_LLVM_COMPILER)
	// save current PGO profiling data and terminate immediately
	PGO_WriteFile();
	TerminateProcess(GetCurrentProcess(), 0);
	return;
#else

	RequestEngineExit(TEXT("Win RequestExit"));

	if (!UE::IsGCLocked())
	{
		FCoreDelegates::GetApplicationWillTerminateDelegate().Broadcast();
	}

	if (Force)
	{
		// Force immediate exit. In case of an error set the exit code to 3.
		// Dangerous because config code isn't flushed, global destructors aren't called, etc.
		// Suppress abort message and MS reports.
		//_set_abort_behavior( 0, _WRITE_ABORT_MSG | _CALL_REPORTFAULT );
		//abort();

		// Make sure the log is flushed.
		if (GLog)
		{
			GLog->Flush();
		}
	
		if (UE::Core::Private::CVarWindowsRaiseExceptionsOnFaultTermination->GetBool())
		{
			if (ReturnCode != 0)
			{
				RaiseException(1);
			}
		}

		TerminateProcess(GetCurrentProcess(), ReturnCode);
	}
	else
	{
		// Tell the platform specific code we want to exit cleanly from the main loop.
		PostQuitMessage(ReturnCode);
	}
#endif
}

const TCHAR* FWindowsPlatformMisc::GetSystemErrorMessage(TCHAR* OutBuffer, int32 BufferCount, int32 Error)
{
	check(OutBuffer && BufferCount);
	*OutBuffer = TEXT('\0');
	if (Error == 0)
	{
		Error = GetLastError();
	}
	FormatMessage( FORMAT_MESSAGE_FROM_SYSTEM, NULL, Error, MAKELANGID(LANG_NEUTRAL, SUBLANG_DEFAULT), OutBuffer, BufferCount, NULL );
	TCHAR* Found = FCString::Strchr(OutBuffer,TEXT('\r'));
	if (Found)
	{
		*Found = TEXT('\0');
	}
	Found = FCString::Strchr(OutBuffer,TEXT('\n'));
	if (Found)
	{
		*Found = TEXT('\0');
	}
	return OutBuffer;
}

void FWindowsPlatformMisc::CreateGuid(FGuid& Result)
{
	if (AutoRTFM::IsTransactional())
	{
		Result = AutoRTFM::Open([]
		{
			FGuid Guid;
			verify( CoCreateGuid( (GUID*)&Guid )==S_OK );
			return Guid;
		});
	}
	else
	{
		verify( CoCreateGuid( (GUID*)&Result )==S_OK );
	}
}

class FWindowsDialog
{
private:
	/**
	 * Calculates button position and size, localize button text.
	 * @param HandleWnd handle to dialog window
	 * @param Text button text to localize
	 * @param DlgItemId dialog item id
	 * @param PositionX current button position (x coord)
	 * @param PositionY current button position (y coord)
	 * @return true if succeeded
	 */
	static bool SetDlgItem(HWND HandleWnd, const TCHAR* Text, int32 DlgItemId, float DPIScale, int32* PositionX, int32* PositionY)
	{
		SIZE SizeButton;

		HDC DC = CreateCompatibleDC(NULL);
		GetTextExtentPoint32(DC, Text, wcslen(Text), &SizeButton);
		DeleteDC(DC);
		DC = NULL;

		SizeButton.cx += (int)(14 * DPIScale);
		SizeButton.cy += (int)(8 * DPIScale);
		SizeButton.cx = FMath::Max((int)(73 * DPIScale), (int)SizeButton.cx);
		SizeButton.cy = FMath::Max((int)(21 * DPIScale), (int)SizeButton.cy);

		HWND Handle = GetDlgItem(HandleWnd, DlgItemId);
		if (Handle)
		{
			*PositionX -= (SizeButton.cx + (int)(7 * DPIScale));
			SetWindowPos(Handle, HWND_TOP, *PositionX, *PositionY - SizeButton.cy, SizeButton.cx, SizeButton.cy, 0);
			SetDlgItemText(HandleWnd, DlgItemId, Text);

			return true;
		}

		return false;
	}

	static float MessageBoxDlgGetDPI(HWND HandleWnd)
	{
		HMODULE User32Module = GetModuleHandle(L"user32.dll");
		if (User32Module == nullptr) { return 1.0f; }

		typedef UINT(WINAPI* LPGetDpiForWindow)(HWND Hwnd);
		LPGetDpiForWindow GetDpiForWindow = (LPGetDpiForWindow)(void*)GetProcAddress(User32Module, "GetDpiForWindow");
		if (GetDpiForWindow == nullptr) { return 1.0f; }

		return static_cast<float>(GetDpiForWindow(HandleWnd)) / USER_DEFAULT_SCREEN_DPI;
	}

	static void SetWindowStyleFlags(HWND HandleWnd, LONG Flags, bool bEnabled)
	{
		LONG Style = GetWindowLong(HandleWnd, GWL_STYLE);
		if (bEnabled)
		{
			Style |= Flags;
		}
		else
		{
			Style &= ~Flags;
		}
		SetWindowLong(HandleWnd, GWL_STYLE, Style);
	}

	static SIZE MeasureText(HWND HandleWnd, LPCWSTR Text, int TextLength)
	{
		HDC DC = CreateCompatibleDC(NULL);
		HFONT Font = (HFONT)SendMessageW(HandleWnd, WM_GETFONT, 0, 0);
		SelectObject(DC, Font);
		RECT TextRect{};
		DrawTextW(DC, Text, TextLength, &TextRect, DT_CALCRECT | DT_EDITCONTROL | DT_LEFT | DT_EXPANDTABS);
		DeleteDC(DC);

		SIZE TextSize;
		TextSize.cx = TextRect.right - TextRect.left;
		TextSize.cy = TextRect.bottom - TextRect.top;
		return TextSize;
	}

	void UpdateEditTextScrollbar(HWND HandleWnd, float DPIScale)
	{
		RECT MessageRect;
		GetWindowRect(HandleWnd, &MessageRect);
		SIZE MessageSize;
		MessageSize.cx = MessageRect.right - MessageRect.left - (int)(20 * DPIScale);
		MessageSize.cy = MessageRect.bottom - MessageRect.top - (int)(16 * DPIScale);

		SIZE TextSize = MeasureText(HandleWnd, *Text, Text.Len());

		bool bNeedsHScroll = TextSize.cx > MessageSize.cx;
		SetWindowStyleFlags(HandleWnd, WS_HSCROLL, bNeedsHScroll);
		bool bNeedsVScroll = TextSize.cy > MessageSize.cy;
		SetWindowStyleFlags(HandleWnd, WS_VSCROLL, bNeedsVScroll);
	}

	void Close(EAppReturnType::Type NewResult)
	{
		WasClosed = true;
		Result = NewResult;
		DestroyWindow(DialogHwnd);
	}

	bool OnInitDialog()
	{
		// Sets most bottom and most right position to begin button placement
		POINT Point;

		GetWindowRect(DialogHwnd, &DefaultWindowRect);
		GetClientRect(DialogHwnd, &ClientRect);
		WasClosed = false;

		Point.x = ClientRect.right;
		Point.y = ClientRect.bottom;

		float DPIScale = MessageBoxDlgGetDPI(DialogHwnd);

		int32 PositionX = Point.x - (int)(5 * DPIScale);
		int32 PositionY = Point.y - (int)(10 * DPIScale);

		// Localize dialog buttons, sets position and size.
		FString CancelString;
		FString RetryString;
		FString ContinueString;
		FString NoToAllString;
		FString NoString;
		FString YesToAllString;
		FString YesString;
		FString OKString;

		// The Localize* functions will return the Key if a dialog is presented before the config system is initialized.
		// Instead, we use hard-coded strings if config is not yet initialized.
		if (!GConfig)
		{
			CancelString = TEXT("Cancel");
			RetryString = TEXT("Retry");
			ContinueString = TEXT("Continue");
			NoToAllString = TEXT("No to All");
			NoString = TEXT("No");
			YesToAllString = TEXT("Yes to All");
			YesString = TEXT("Yes");
			OKString = TEXT("OK");
		}
		else
		{
			CancelString = NSLOCTEXT("UnrealEd", "Cancel", "Cancel").ToString();
			RetryString = NSLOCTEXT("UnrealEd", "Retry", "Retry").ToString();
			ContinueString = NSLOCTEXT("UnrealEd", "Continue", "Continue").ToString();
			NoToAllString = NSLOCTEXT("UnrealEd", "NoToAll", "No to All").ToString();
			NoString = NSLOCTEXT("UnrealEd", "No", "No").ToString();
			YesToAllString = NSLOCTEXT("UnrealEd", "YesToAll", "Yes to All").ToString();
			YesString = NSLOCTEXT("UnrealEd", "Yes", "Yes").ToString();
			OKString = NSLOCTEXT("UnrealEd", "OK", "OK").ToString();
		}
		SetDlgItem(DialogHwnd, *ContinueString, IDC_CONTINUE, DPIScale, &PositionX, &PositionY);
		SetDlgItem(DialogHwnd, *RetryString, IDC_RETRY, DPIScale, &PositionX, &PositionY);
		SetDlgItem(DialogHwnd, *CancelString, IDC_CANCEL, DPIScale, &PositionX, &PositionY);
		SetDlgItem(DialogHwnd, *NoToAllString, IDC_NOTOALL, DPIScale, &PositionX, &PositionY);
		SetDlgItem(DialogHwnd, *NoString, IDC_NO_B, DPIScale, &PositionX, &PositionY);
		SetDlgItem(DialogHwnd, *YesToAllString, IDC_YESTOALL, DPIScale, &PositionX, &PositionY);
		SetDlgItem(DialogHwnd, *YesString, IDC_YES, DPIScale, &PositionX, &PositionY);
		SetDlgItem(DialogHwnd, *OKString, IDC_OK, DPIScale, &PositionX, &PositionY);

		SetDlgItemText(DialogHwnd, IDC_MESSAGE, *Text);
		SetWindowText(DialogHwnd, *Caption);

		HWND MessageHandle = GetDlgItem(DialogHwnd, IDC_MESSAGE);
		UpdateEditTextScrollbar(MessageHandle, DPIScale);

		// If parent window exist, get it handle and make it foreground.
		HWND ParentWindow = GetTopWindow(DialogHwnd);
		if (ParentWindow)
		{
			SetWindowPos(ParentWindow, HWND_TOPMOST, 0, 0, 0, 0, SWP_NOMOVE | SWP_NOSIZE);
		}

		SetForegroundWindow(DialogHwnd);
		SetWindowPos(DialogHwnd, HWND_TOPMOST, 0, 0, 0, 0, SWP_NOMOVE | SWP_NOSIZE);


		// Windows are foreground, make them not top most.
		SetWindowPos(DialogHwnd, HWND_NOTOPMOST, 0, 0, 0, 0, SWP_NOMOVE | SWP_NOSIZE);
		if (ParentWindow)
		{
			SetWindowPos(ParentWindow, HWND_NOTOPMOST, 0, 0, 0, 0, SWP_NOMOVE | SWP_NOSIZE);
		}

		// Resize to fit text
		int Width = DefaultWindowRect.right - DefaultWindowRect.left;
		int Height = DefaultWindowRect.bottom - DefaultWindowRect.top;
		SIZE TextSize = MeasureText(MessageHandle, *Text, Text.Len());

		HMONITOR Monitor = MonitorFromWindow(DialogHwnd, MONITOR_DEFAULTTONEAREST);
		MONITORINFO MonInfo;
		MonInfo.cbSize = sizeof(MONITORINFO);
		GetMonitorInfo(Monitor, &MonInfo);
		int MonitorWidth = MonInfo.rcMonitor.right - MonInfo.rcMonitor.left;
		int MonitorHeight = MonInfo.rcMonitor.bottom - MonInfo.rcMonitor.top;

		int NewWidth = FMath::Clamp((int)((float)TextSize.cx + 100.0f * DPIScale), Width, (int)((float)MonitorWidth * 0.8f));
		int NewHeight = FMath::Clamp((int)((float)TextSize.cy + 165.0f * DPIScale), Height, (int)((float)MonitorHeight * 0.8f));
		int NewLeft = DefaultWindowRect.left - (NewWidth - Width) / 2;
		int NewTop = DefaultWindowRect.top - (NewHeight - Height) / 2;
		SetWindowPos(DialogHwnd, HWND_NOTOPMOST, NewLeft, NewTop, NewWidth, NewHeight, SWP_NOZORDER);

		return true;
	}

	void SetDialogTextInClipboard(HWND HandleWnd)
	{
		size_t TextByteCount = Text.GetAllocatedSize();
		HGLOBAL StrClipboardMemory = GlobalAlloc(GMEM_MOVEABLE, TextByteCount);
		if (StrClipboardMemory == nullptr) { return; }

		void* StrTarget = GlobalLock(StrClipboardMemory);
		if (StrTarget == nullptr) { return; }
		memcpy(StrTarget, *Text, TextByteCount);
		((char*)StrTarget)[TextByteCount] = 0;
		GlobalUnlock(StrClipboardMemory);

		if (!OpenClipboard(HandleWnd)) { return; }
		if (!EmptyClipboard()) { return; }
		SetClipboardData(CF_UNICODETEXT, StrClipboardMemory);
		CloseClipboard();
	}

	/**
	 * Callback for MessageBoxExt dialog (allowing for Yes to all / No to all )
	 * @return		One of EAppReturnType::Yes, EAppReturnType::YesAll, EAppReturnType::No, EAppReturnType::NoAll, EAppReturnType::Cancel.
	 */
	PTRINT MessageBoxDlgProc(HWND HandleWnd, uint32 Message, WPARAM WParam, LPARAM LParam)
	{
		switch (Message)
		{
			case WM_INITDIALOG:
			{
				return OnInitDialog();
			}
			case WM_DESTROY:
			{
				return true;
			}
			case WM_CLOSE:
			{
				Close(Result);
				return true;
			}
			case WM_COMMAND:
			{
				switch (LOWORD(WParam))
				{
					case IDC_OK:
						Close(EAppReturnType::Ok);
						break;
					case IDC_YES:
						Close(EAppReturnType::Yes);
						break;
					case IDC_YESTOALL:
						Close(EAppReturnType::YesAll);
						break;
					case IDC_NO_B:
						Close(EAppReturnType::No);
						break;
					case IDC_NOTOALL:
						Close(EAppReturnType::NoAll);
						break;
					case IDC_RETRY:
						Close(EAppReturnType::Retry);
						break;
					case IDC_CONTINUE:
						Close(EAppReturnType::Continue);
						break;
					case IDC_CANCEL:
						if (CancelButtonEnabled)
						{
							Close(EAppReturnType::Cancel);
							break;
						}
						break;
					case IDC_COPY:
					{
						SetDialogTextInClipboard(HandleWnd);
						break;
					}
					default:
						return false;
				}
				return true;
			}
			case WM_CTLCOLORSTATIC:
			{
				if ((HWND)LParam == GetDlgItem(HandleWnd, IDC_MESSAGE))
				{
					SetBkMode((HDC)WParam, TRANSPARENT);
					return (LRESULT)(GetSysColorBrush(COLOR_WINDOW));
				}
				else
				{
					return false;
				}
			}
			case WM_PAINT:
			{
				PAINTSTRUCT Paint;
				HDC Hdc = BeginPaint(HandleWnd, &Paint);

				float DPIScale = MessageBoxDlgGetDPI(HandleWnd);

				RECT MainAreaRect;
				MainAreaRect = ClientRect;
				const int ActionBarHeight = (int)(45 * DPIScale);
				MainAreaRect.bottom -= ActionBarHeight;
				FillRect(Hdc, &MainAreaRect, GetSysColorBrush(COLOR_WINDOW));

				/*HWND MessageHandle = GetDlgItem(HandleWnd, IDC_MESSAGE);
				RECT MessageRect;
				GetWindowRect(MessageHandle, &MessageRect);
				MessageRect.right = MessageRect.right - MessageRect.left;
				MessageRect.bottom = MessageRect.bottom - MessageRect.top;
				MessageRect.left = MessageRect.top = 15;
				HFONT Font = (HFONT)SendMessageW(MessageHandle, WM_GETFONT, 0, 0);
				SelectObject(Hdc, Font);
				DrawTextW(Hdc, GDialogState.Text, wcslen(GDialogState.Text), &MainAreaRect, DT_EDITCONTROL | DT_LEFT | DT_EXPANDTABS);*/

				EndPaint(HandleWnd, &Paint);
				return false;
			}
			case WM_SIZE:
			{
				RECT PrevDialogRect = ClientRect;
				GetClientRect(HandleWnd, &ClientRect);

				int ControlsStretch[] =
				{
					IDC_MESSAGE
				};

				for (const int ControlId : ControlsStretch)
				{
					HWND ControlHandle = GetDlgItem(HandleWnd, ControlId);
					if (ControlHandle == nullptr) { continue; }

					RECT PrevMessageRect;
					GetWindowRect(ControlHandle, &PrevMessageRect);
					ScreenToClient(HandleWnd, (POINT*)&PrevMessageRect.left);
					ScreenToClient(HandleWnd, (POINT*)&PrevMessageRect.right);

					SetWindowPos(ControlHandle, 0,
						PrevMessageRect.left,
						PrevMessageRect.top,
						(PrevMessageRect.right - PrevMessageRect.left) + (ClientRect.right - PrevDialogRect.right),
						(PrevMessageRect.bottom - PrevMessageRect.top) + (ClientRect.bottom - PrevDialogRect.bottom), SWP_NOZORDER);
				}

				int ControlsBottomRight[] =
				{
					IDC_OK, IDC_YES, IDC_YESTOALL, IDC_NO_B, IDC_NOTOALL, IDC_CANCEL, IDC_RETRY, IDC_CONTINUE
				};

				for (const int ControlId : ControlsBottomRight)
				{
					HWND ControlHandle = GetDlgItem(HandleWnd, ControlId);
					if (ControlHandle == nullptr) { continue; }

					RECT PrevMessageRect;
					GetWindowRect(ControlHandle, &PrevMessageRect);
					ScreenToClient(HandleWnd, (POINT*)&PrevMessageRect.left);
					ScreenToClient(HandleWnd, (POINT*)&PrevMessageRect.right);

					SetWindowPos(ControlHandle, 0,
						PrevMessageRect.left + (ClientRect.right - PrevDialogRect.right),
						PrevMessageRect.top + (ClientRect.bottom - PrevDialogRect.bottom),
						(PrevMessageRect.right - PrevMessageRect.left),
						(PrevMessageRect.bottom - PrevMessageRect.top), SWP_NOZORDER);
				}

				float DPIScale = MessageBoxDlgGetDPI(HandleWnd);

				HWND MessageHandle = GetDlgItem(HandleWnd, IDC_MESSAGE);
				UpdateEditTextScrollbar(MessageHandle, DPIScale);

				InvalidateRect(HandleWnd, nullptr, true);
			}
			return true;
			case WM_GETMINMAXINFO:
			{
				MINMAXINFO* Info = (MINMAXINFO*)LParam;

				Info->ptMinTrackSize.x = DefaultWindowRect.right - DefaultWindowRect.left;
				Info->ptMinTrackSize.y = DefaultWindowRect.bottom - DefaultWindowRect.top;
			}
			return true;
			case WM_DPICHANGED:
				/*{
					float NewScale = LOWORD(WParam) / static_cast<float>(USER_DEFAULT_SCREEN_DPI);
					RECT* SuggestedRect = (RECT*)LParam;
					SetWindowPos(HandleWnd,
						HWND_NOTOPMOST,
						SuggestedRect->left,
						SuggestedRect->top,
						SuggestedRect->right - SuggestedRect->left,
						SuggestedRect->bottom - SuggestedRect->top,
						SWP_NOZORDER);

					//https://github.com/microsoft/Windows-classic-samples/blob/main/Samples/DPIAwarenessPerWindow/client/DpiAwarenessContext.cpp

				}*/
				return true;
			default:
				return false;
		}
	}

	FWindowsDialog() {}


	FString Text;
	FString Caption;

	/**
	 * Used to indicate whether a 'Cancel' button is present and
	 * thus 'Esc should be accepted as a hotkey.
	 */
	bool CancelButtonEnabled = false;


	HWND DialogHwnd = NULL;
	RECT ClientRect = {};
	RECT DefaultWindowRect = {};
	bool WasClosed = false;
	EAppReturnType::Type Result = EAppReturnType::Cancel;

	EAppReturnType::Type Show(HWND ParentWindowHandle, int Template)
	{
		HACCEL AcceleratorHandle = LoadAccelerators(hInstance, MAKEINTRESOURCE(IDR_ACCEL1));

		DialogHwnd = CreateDialogParam(GetModuleHandle(NULL), MAKEINTRESOURCE(Template), ParentWindowHandle, 
			[](HWND HandleWnd, uint32 Message, WPARAM WParam, LPARAM LParam) {
				FWindowsDialog* Instance;
				if (Message == WM_INITDIALOG)
				{
					Instance = (FWindowsDialog*)LParam;
					Instance->DialogHwnd = HandleWnd;
					SetWindowLongPtr(HandleWnd, DWLP_USER, (LPARAM)Instance);
				}
				else
				{
					Instance = (FWindowsDialog*)GetWindowLongPtr(HandleWnd, DWLP_USER);
				}

				return Instance->MessageBoxDlgProc(HandleWnd, Message, WParam, LParam);
			}, (LPARAM)this);

		if (!DialogHwnd)
		{
			DWORD LastError = GetLastError();
			TCHAR ErrorBuffer[1024];
			FWindowsPlatformMisc::GetSystemErrorMessage(ErrorBuffer, UE_ARRAY_COUNT(ErrorBuffer), LastError);
			UE_LOGF(LogWindows, Error, "Failed to create dialog. %ls Error: 0x%X (%u)", ErrorBuffer, LastError, LastError);
			return Result;
		}

		ShowWindow(DialogHwnd, SW_SHOW);
		MSG Msg;
		while (!WasClosed && GetMessageW(&Msg, NULL, 0, 0))
		{
			bool bIsTextEditMsg = Msg.hwnd == GetDlgItem(DialogHwnd, IDC_MESSAGE);
			bool bCheckAccelerators = DialogHwnd == Msg.hwnd || (IsChild(DialogHwnd, Msg.hwnd) && !bIsTextEditMsg);
			if (!(bCheckAccelerators && TranslateAccelerator(DialogHwnd, AcceleratorHandle, &Msg)) &&
				!IsDialogMessage(DialogHwnd, &Msg))
			{
				TranslateMessage(&Msg);
				DispatchMessage(&Msg);
			}
		}

		DestroyAcceleratorTable(AcceleratorHandle);

		return Result;
	}

public:
	static EAppReturnType::Type Show(HWND ParentWindowHandle, EAppMsgType::Type MsgType, const FString& Text, const FString& Caption)
	{
		FWindowsDialog Instance {};
		int MaxLength = 1 << 20;
		if (Text.Len() > MaxLength)
		{
			Instance.Text = Text.Left(MaxLength) + TEXT("\r\n[Message truncated]");
		}
		else
		{
			Instance.Text = Text;
		}
		Instance.Caption = Caption;

		switch (MsgType)
		{
			case EAppMsgType::Ok:
			{
				Instance.CancelButtonEnabled = false;
				Instance.Result = EAppReturnType::Ok; // default option used when dialog is closed
				return Instance.Show(ParentWindowHandle, IDD_OK);
			}
			case EAppMsgType::YesNo:
			{
				Instance.CancelButtonEnabled = false;
				Instance.Result = EAppReturnType::No;
				return Instance.Show(ParentWindowHandle, IDD_YESNO);
			}
			case EAppMsgType::OkCancel:
			{
				Instance.CancelButtonEnabled = true;
				Instance.Result = EAppReturnType::Cancel;
				return Instance.Show(ParentWindowHandle, IDD_OKCANCEL);
			}
			case EAppMsgType::YesNoCancel:
			{
				Instance.CancelButtonEnabled = true;
				Instance.Result = EAppReturnType::Cancel;
				return Instance.Show(ParentWindowHandle, IDD_YESNOCANCEL);
			}
			case EAppMsgType::CancelRetryContinue:
			{
				Instance.CancelButtonEnabled = true;
				Instance.Result = EAppReturnType::Cancel;
				return Instance.Show(ParentWindowHandle, IDD_CANCELRETRYCONTINUE);
			}
			case EAppMsgType::YesNoYesAllNoAll:
			{
				Instance.CancelButtonEnabled = false;
				Instance.Result = EAppReturnType::No;
				return Instance.Show(ParentWindowHandle, IDD_YESNO2ALL);
			}
			case EAppMsgType::YesNoYesAllNoAllCancel:
			{
				Instance.CancelButtonEnabled = true;
				Instance.Result = EAppReturnType::Cancel;
				return Instance.Show(ParentWindowHandle, IDD_YESNO2ALLCANCEL);
			}
			case EAppMsgType::YesNoYesAll:
			{
				Instance.CancelButtonEnabled = false;
				Instance.Result = EAppReturnType::No;
				return Instance.Show(ParentWindowHandle, IDD_YESNOYESTOALL);
			}
			default:
				return EAppReturnType::Cancel;
		}
	}
};


EAppReturnType::Type FWindowsPlatformMisc::MessageBoxExt( EAppMsgType::Type MsgType, const TCHAR* Text, const TCHAR* Caption )
{
	if (UE::Core::Private::CVarDisableWindowsMessageBox.GetValueOnAnyThread() == true)
	{
		UE_LOGF(LogWindows, Warning, "FWindowsPlatformMisc::MessageBoxExt: A popup creation was auto-cancelled (%ls)", Caption);
		return EAppReturnType::Cancel;
	}

	FSlowHeartBeatScope SuspendHeartBeat;

	HWND ParentWindow = (HWND)NULL;
	
	FString PlatformText = FString(Text);
	PlatformText.ReplaceInline(TEXT("\r"), TEXT(""));
	PlatformText.ReplaceInline(TEXT("\n"), TEXT("\r\n"));

	return FWindowsDialog::Show(ParentWindow, MsgType, PlatformText, FString(Caption));
}

static bool HandleGameExplorerIntegration()
{
	// skip this if running on WindowsServer (we get rare crashes that seem to stem from Windows Server builds, where GameExplorer isn't particularly useful)
	if (FPlatformProperties::SupportsWindowedMode() && !IsWindowsServer())
	{
		TCHAR AppPath[MAX_PATH];
		GetModuleFileName(NULL, AppPath, MAX_PATH - 1);

		// Initialize COM. We only want to do this once and not override settings of previous calls.
		if (!FWindowsPlatformMisc::CoInitialize())
		{
			return false;
		}
		
		// check to make sure we are able to run, based on parental rights
		IGameExplorer* GameExp;
		HRESULT hr = CoCreateInstance(__uuidof(GameExplorer), NULL, CLSCTX_INPROC_SERVER, __uuidof(IGameExplorer), (void**) &GameExp);

		BOOL bHasAccess = 1;
		BSTR AppPathBSTR = SysAllocString(AppPath);

		// @todo: This will allow access if the CoCreateInstance fails, but it should probaly disallow 
		// access if OS is Vista and it fails, succeed for XP
		if (SUCCEEDED(hr) && GameExp)
		{
			GameExp->VerifyAccess(AppPathBSTR, &bHasAccess);
		}


		// Guid for testing GE (un)installation
		static const GUID GEGuid = 
		{ 0x7089dd1d, 0xfe97, 0x4cc8, { 0x8a, 0xac, 0x26, 0x3e, 0x44, 0x1f, 0x3c, 0x42 } };

		// add the game to the game explorer if desired
		if (FParse::Param( FCommandLine::Get(), TEXT("installge")))
		{
			if (bHasAccess && GameExp)
			{
				BSTR AppDirBSTR = SysAllocString(FPlatformProcess::BaseDir());
				GUID Guid = GEGuid;
				hr = GameExp->AddGame(AppPathBSTR, AppDirBSTR, FParse::Param( FCommandLine::Get(), TEXT("allusers")) ? GIS_ALL_USERS : GIS_CURRENT_USER, &Guid);

				bool bWasSuccessful = false;
				// if successful
				if (SUCCEEDED(hr))
				{
					// get location of app local dir
					TCHAR UserPath[MAX_PATH];
					SHGetFolderPath(NULL, CSIDL_LOCAL_APPDATA | CSIDL_FLAG_CREATE, NULL, SHGFP_TYPE_CURRENT, UserPath);

					// convert guid to a string
					TCHAR GuidDir[MAX_PATH];
					CA_SUPPRESS(6031)
					StringFromGUID2(GEGuid, GuidDir, MAX_PATH - 1);

					// make the base path for all tasks
					FString BaseTaskDirectory = FString(UserPath) + TEXT("\\Microsoft\\Windows\\GameExplorer\\") + GuidDir;

					// make full paths for play and support tasks
					FString PlayTaskDirectory = BaseTaskDirectory + TEXT("\\PlayTasks");
					FString SupportTaskDirectory = BaseTaskDirectory + TEXT("\\SupportTasks");
				
					// make sure they exist
					IFileManager::Get().MakeDirectory(*PlayTaskDirectory, true);
					IFileManager::Get().MakeDirectory(*SupportTaskDirectory, true);

					// interface for creating a shortcut
					IShellLink* Link;
					hr = CoCreateInstance(CLSID_ShellLink, NULL, CLSCTX_INPROC_SERVER,	IID_IShellLink, (void**)&Link);

					// get the persistent file interface of the link
					IPersistFile* LinkFile;
					Link->QueryInterface(IID_IPersistFile, (void**)&LinkFile);

					Link->SetPath(AppPath);

					// create all of our tasks

					// first is just the game
					Link->SetArguments(TEXT(""));
					Link->SetDescription(TEXT("Play"));
					IFileManager::Get().MakeDirectory(*(PlayTaskDirectory + TEXT("\\0")), true);
					LinkFile->Save(*(PlayTaskDirectory + TEXT("\\0\\Play.lnk")), true);

					Link->SetArguments(TEXT("editor"));
					Link->SetDescription(TEXT("Editor"));
					IFileManager::Get().MakeDirectory(*(PlayTaskDirectory + TEXT("\\1")), true);
					LinkFile->Save(*(PlayTaskDirectory + TEXT("\\1\\Editor.lnk")), true);

					LinkFile->Release();
					Link->Release();

					IUniformResourceLocator* InternetLink;
					CA_SUPPRESS(6031)
					CoCreateInstance (CLSID_InternetShortcut, NULL, 
						CLSCTX_INPROC_SERVER, IID_IUniformResourceLocator, (LPVOID*) &InternetLink);

					InternetLink->QueryInterface(IID_IPersistFile, (void**)&LinkFile);

					// make an internet shortcut
					InternetLink->SetURL(TEXT("http://www.unrealtournament3.com/"), 0);
					IFileManager::Get().MakeDirectory(*(SupportTaskDirectory + TEXT("\\0")), true);
					LinkFile->Save(*(SupportTaskDirectory + TEXT("\\0\\UT3.url")), true);

					LinkFile->Release();
					InternetLink->Release();
				}

				if ( SUCCEEDED(hr) ) 
				{
					FMessageDialog::Open( EAppMsgType::Ok, NSLOCTEXT("WindowsPlatform", "GameExplorerInstallationSuccessful", "GameExplorer installation was successful, quitting now.") );
				}
				else
				{
					FMessageDialog::Open( EAppMsgType::Ok, NSLOCTEXT("WindowsPlatform", "GameExplorerInstallationFailed", "GameExplorer installation was a failure, quitting now.") );
				}

				SysFreeString(AppDirBSTR);
			}
			else
			{
				FMessageDialog::Open( EAppMsgType::Ok, NSLOCTEXT("WindowsPlatform", "GameExplorerInstallationFailedDoToAccessPermissions", "GameExplorer installation failed because you don't have access (check parental control levels and that you are running XP). You should not need Admin access"));
			}

			// free the string and shutdown COM
			SysFreeString(AppPathBSTR);
			SAFE_RELEASE(GameExp);
			FWindowsPlatformMisc::CoUninitialize();

			return false;
		}
		else if (FParse::Param( FCommandLine::Get(), TEXT("uninstallge")))
		{
			if (GameExp)
			{
				hr = GameExp->RemoveGame(GEGuid);
				if ( SUCCEEDED(hr) ) 
				{
					FMessageDialog::Open( EAppMsgType::Ok, NSLOCTEXT("WindowsPlatform", "GameExplorerUninstallationSuccessful", "GameExplorer uninstallation was successful, quitting now.") );
				}
				else
				{
					FMessageDialog::Open( EAppMsgType::Ok, NSLOCTEXT("WindowsPlatform", "GameExplorerUninstallationFailed", "GameExplorer uninstallation was a failure, quitting now.") );
				}
			}
			else
			{
				FMessageDialog::Open( EAppMsgType::Ok, NSLOCTEXT("WindowsPlatform", "GameExplorerUninstallationFailedDoToNotRunningVista", "GameExplorer uninstallation failed because you are probably not running Vista."));
			}

			// free the string and shutdown COM
			SysFreeString(AppPathBSTR);
			SAFE_RELEASE(GameExp);
			FWindowsPlatformMisc::CoUninitialize();

			return false;
		}

		// free the string and shutdown COM
		SysFreeString(AppPathBSTR);
		SAFE_RELEASE(GameExp);
		FWindowsPlatformMisc::CoUninitialize();

		// if we don't have access, we must quit ASAP after showing a message
		if (!bHasAccess)
		{
			FMessageDialog::Open( EAppMsgType::Ok, NSLOCTEXT("UnrealEd", "Error_ParentalControls", "The current level of parental controls do not allow you to run this game." ) );
			return false;
		}
	}
	return true;
}

#if WITH_FIREWALL_SUPPORT
/** 
 * Get the INetFwProfile interface for current profile
 */
INetFwProfile* GetFirewallProfile( void )
{
	HRESULT hr;
	INetFwMgr* pFwMgr = NULL;
	INetFwPolicy* pFwPolicy = NULL;
	INetFwProfile* pFwProfile = NULL;

	// Create an instance of the Firewall settings manager
	hr = CoCreateInstance( __uuidof( NetFwMgr ), NULL, CLSCTX_INPROC_SERVER, __uuidof( INetFwMgr ), ( void** )&pFwMgr );
	if( SUCCEEDED( hr ) )
	{
		hr = pFwMgr->get_LocalPolicy( &pFwPolicy );
		if( SUCCEEDED( hr ) )
		{
			pFwPolicy->get_CurrentProfile( &pFwProfile );
		}
	}

	// Cleanup
	if( pFwPolicy )
	{
		pFwPolicy->Release();
	}
	if( pFwMgr )
	{
		pFwMgr->Release();
	}

	return( pFwProfile );
}
#endif

static bool HandleFirewallIntegration()
{
#if WITH_FIREWALL_SUPPORT
	// only do with with the given commandlines
	if( !(FParse::Param( FCommandLine::Get(), TEXT( "installfw" ) ) || FParse::Param( FCommandLine::Get(), TEXT( "uninstallfw" ) )) )
#endif
	{
		return true; // allow the game to continue;
	}
#if WITH_FIREWALL_SUPPORT

	TCHAR AppPath[MAX_PATH];

	GetModuleFileName( NULL, AppPath, MAX_PATH - 1 );
	BSTR bstrGameExeFullPath = SysAllocString( AppPath );
	BSTR bstrFriendlyAppName = SysAllocString( TEXT( "Unreal Tournament 3" ) );

	if( bstrGameExeFullPath && bstrFriendlyAppName )
	{
		HRESULT hr = S_OK;
				
		if( FWindowsPlatformMisc::CoInitialize() )
		{
			INetFwProfile* pFwProfile = GetFirewallProfile();
			if( pFwProfile )
			{
				INetFwAuthorizedApplications* pFwApps = NULL;

				hr = pFwProfile->get_AuthorizedApplications( &pFwApps );
				if( SUCCEEDED( hr ) && pFwApps ) 
				{
					// add the game to the game explorer if desired
					if( FParse::Param( CmdLine, TEXT( "installfw" ) ) )
					{
						INetFwAuthorizedApplication* pFwApp = NULL;

						// Create an instance of an authorized application.
						hr = CoCreateInstance( __uuidof( NetFwAuthorizedApplication ), NULL, CLSCTX_INPROC_SERVER, __uuidof( INetFwAuthorizedApplication ), ( void** )&pFwApp );
						if( SUCCEEDED( hr ) && pFwApp )
						{
							// Set the process image file name.
							hr = pFwApp->put_ProcessImageFileName( bstrGameExeFullPath );
							if( SUCCEEDED( hr ) )
							{
								// Set the application friendly name.
								hr = pFwApp->put_Name( bstrFriendlyAppName );
								if( SUCCEEDED( hr ) )
								{
									// Add the application to the collection.
									hr = pFwApps->Add( pFwApp );
								}
							}

							pFwApp->Release();
						}
					}
					else if( FParse::Param( CmdLine, TEXT( "uninstallfw" ) ) )
					{
						// Remove the application from the collection.
						hr = pFwApps->Remove( bstrGameExeFullPath );
					}

					pFwApps->Release();
				}

				pFwProfile->Release();
			}

			FWindowsPlatformMisc::CoUninitialize();
		}

		SysFreeString( bstrFriendlyAppName );
		SysFreeString( bstrGameExeFullPath );
	}
	return false; // terminate the game
#endif // WITH_FIREWALL_SUPPORT
}

static bool HandleFirstInstall()
{
	if (FParse::Param( FCommandLine::Get(), TEXT("firstinstall")))
	{
		GLog->Flush();

		// Flush config to ensure culture changes are written to disk.
		GConfig->Flush(false);

		return false; // terminate the game
	}
	return true; // allow the game to continue;
}

bool FWindowsPlatformMisc::CommandLineCommands()
{
	return HandleFirstInstall() && HandleGameExplorerIntegration() && HandleFirewallIntegration();
}

bool FWindowsPlatformMisc::VerifyWindowsVersion(uint32 MajorVersion, uint32 MinorVersion, uint32 BuildNumber)
{
	OSVERSIONINFOEX Version;
	Version.dwOSVersionInfoSize = sizeof(OSVERSIONINFOEX);
	Version.dwMajorVersion = MajorVersion;
	Version.dwMinorVersion = MinorVersion;
	Version.dwBuildNumber  = BuildNumber;

	ULONGLONG ConditionMask = 0;
	ConditionMask = VerSetConditionMask(ConditionMask, VER_MAJORVERSION, VER_GREATER_EQUAL);
	ConditionMask = VerSetConditionMask(ConditionMask, VER_MINORVERSION, VER_GREATER_EQUAL);
	ConditionMask = VerSetConditionMask(ConditionMask, VER_BUILDNUMBER,  VER_GREATER_EQUAL);

	return !!VerifyVersionInfo(&Version, VER_MAJORVERSION | VER_MINORVERSION | VER_BUILDNUMBER, ConditionMask);
}

bool FWindowsPlatformMisc::IsWine()
{
	struct FWineDetected
	{
		bool bValue;

		FWineDetected()
		{
			HMODULE Handle = GetModuleHandle(TEXT("ntdll.dll"));
			if (Handle == NULL)
			{
				bValue = false;
			}
			else
			{
				void* WineGetVersion = (void*)GetProcAddress(Handle, "wine_get_version");
				bValue = (WineGetVersion != NULL);
			}
		}
	};

	static FWineDetected WineDetected;
	return WineDetected.bValue;
}

bool FWindowsPlatformMisc::IsValidAbsolutePathFormat(const FString& Path)
{
	bool bIsValid = true;
	const FString OnlyPath = FPaths::GetPath(Path);
	if ( OnlyPath.IsEmpty() )
	{
		bIsValid = false;
	}

	// Must begin with a drive letter
	if ( bIsValid && !FChar::IsAlpha(OnlyPath[0]) )
	{
		bIsValid = false;
	}

	// On Windows the path must be absolute, i.e: "D:/" or "D:\\"
	if ( bIsValid && !(Path.Find(TEXT(":/"))==1 || Path.Find(TEXT(":\\"))==1) )
	{
		bIsValid = false;
	}

	// Find any unnamed directory changes
	if ( bIsValid && (Path.Find(TEXT("//"))!=INDEX_NONE || Path.Find(TEXT("\\/"))!=INDEX_NONE || Path.Find(TEXT("/\\"))!=INDEX_NONE || Path.Find(TEXT("\\\\"))!=INDEX_NONE) )
	{
		bIsValid = false;
	}

	// Avoid folders with only a space in their name e.g. "D:\TEMP\ " or "D:\TEMP\ \MyFolder"
	if (bIsValid && Path.Find(TEXT("\\ \\")) != INDEX_NONE || Path.EndsWith(TEXT("\\ ")))
	{
		bIsValid = false;
	}

	// ensure there's no further instances of ':' in the string
	if ( bIsValid && !(Path.Find(TEXT(":"), ESearchCase::IgnoreCase, ESearchDir::FromStart, 2)==INDEX_NONE) )
	{
		bIsValid = false;
	}

	return bIsValid;
}

static void QueryCpuInformation(FProcessorGroupDesc& OutGroupDesc, uint32& OutNumaNodeCount, uint32& OutCoreCount, uint32& OutLogicalProcessorCount, bool bForceSingleNumaNode = false)
{
	GROUP_AFFINITY FilterGroupAffinity = {};

	if (bForceSingleNumaNode)
	{
		PROCESSOR_NUMBER ProcessorNumber = {};
		USHORT NodeNumber = 0;

		GetThreadIdealProcessorEx(GetCurrentThread(), &ProcessorNumber);
		GetNumaProcessorNodeEx(&ProcessorNumber, &NodeNumber);
		GetNumaNodeProcessorMaskEx(NodeNumber, &FilterGroupAffinity);
	}

	OutNumaNodeCount = OutCoreCount = OutLogicalProcessorCount = 0;
	uint8* BufferPtr = nullptr;
	DWORD BufferBytes = 0;

	if (false == GetLogicalProcessorInformationEx(RelationAll, (PSYSTEM_LOGICAL_PROCESSOR_INFORMATION_EX) BufferPtr, &BufferBytes))
	{
		if (GetLastError() == ERROR_INSUFFICIENT_BUFFER)
		{
			BufferPtr = reinterpret_cast<uint8*>(FMemory::Malloc(BufferBytes));

			if (GetLogicalProcessorInformationEx(RelationAll, (PSYSTEM_LOGICAL_PROCESSOR_INFORMATION_EX) BufferPtr, &BufferBytes))
			{
				uint8* InfoPtr = BufferPtr;

				while (InfoPtr < BufferPtr + BufferBytes)
				{
					PSYSTEM_LOGICAL_PROCESSOR_INFORMATION_EX ProcessorInfo = (PSYSTEM_LOGICAL_PROCESSOR_INFORMATION_EX) InfoPtr;

					if (nullptr == ProcessorInfo)
					{
						break;
					}

					if (ProcessorInfo->Relationship == RelationProcessorCore)
					{
						if (bForceSingleNumaNode)
						{
							for (int GroupIdx = 0; GroupIdx < ProcessorInfo->Processor.GroupCount; ++GroupIdx)
							{
								if (FilterGroupAffinity.Group == ProcessorInfo->Processor.GroupMask[GroupIdx].Group)
								{
									KAFFINITY Intersection = FilterGroupAffinity.Mask & ProcessorInfo->Processor.GroupMask[GroupIdx].Mask;

									if (Intersection > 0)
									{
										OutCoreCount++;

										OutLogicalProcessorCount += FMath::CountBits64(Intersection);
									}
								}
							}
						}
						else
						{
							OutCoreCount++;

							for (int GroupIdx = 0; GroupIdx < ProcessorInfo->Processor.GroupCount; ++GroupIdx)
							{
								OutLogicalProcessorCount += FMath::CountBits64(ProcessorInfo->Processor.GroupMask[GroupIdx].Mask);
							}
						}
					}
					if (ProcessorInfo->Relationship == RelationNumaNode)
					{
						OutNumaNodeCount++;
					}

					if (ProcessorInfo->Relationship == RelationGroup)
					{
						OutGroupDesc.NumProcessorGroups = FMath::Min<uint16>(FProcessorGroupDesc::MaxNumProcessorGroups, ProcessorInfo->Group.ActiveGroupCount);
						for(int32 GroupIndex = 0; GroupIndex < OutGroupDesc.NumProcessorGroups; GroupIndex++)
						{
							OutGroupDesc.ThreadAffinities[GroupIndex] = ProcessorInfo->Group.GroupInfo[GroupIndex].ActiveProcessorMask;
						}
					}

					InfoPtr += ProcessorInfo->Size;
				}
			}

			FMemory::Free(BufferPtr);
		}
	}
}

int32 FWindowsPlatformMisc::NumberOfCores()
{
	static int32 CoreCount = 0;
	if (CoreCount > 0)
	{
		return CoreCount;
	}

	FProcessorGroupDesc GroupDesc;
	uint32 NumaNodeCount = 0;
	uint32 NumCores = 0;
	uint32 LogicalProcessorCount = 0;
	QueryCpuInformation(GroupDesc, NumaNodeCount, NumCores, LogicalProcessorCount);

	bool bLimitsInitialized;
	int32 PhysicalCoreLimit;
	int32 LogicalCoreLimit;
	bool bSetPhysicalCountToLogicalCount;
	GetConfiguredCoreLimits(NumCores, LogicalProcessorCount, bLimitsInitialized, PhysicalCoreLimit,
		LogicalCoreLimit, bSetPhysicalCountToLogicalCount);

	CoreCount = bSetPhysicalCountToLogicalCount ? LogicalProcessorCount : NumCores;

	// Optionally limit number of threads (we don't necessarily scale super well with very high core counts)
	if (PhysicalCoreLimit > 0)
	{
		CoreCount = FMath::Min(CoreCount, PhysicalCoreLimit);
	}

	return CoreCount;
}

FProcessorGroupDesc NumberOfProcessorGroupsInternal()
{
	FProcessorGroupDesc GroupDesc;
	uint32 NumaNodeCount = 0;
	uint32 NumCores = 0;
	uint32 LogicalProcessorCount = 0;
	QueryCpuInformation(GroupDesc, NumaNodeCount, NumCores, LogicalProcessorCount);
	return GroupDesc;
}

const FProcessorGroupDesc& FWindowsPlatformMisc::GetProcessorGroupDesc()
{
	static FProcessorGroupDesc GroupDesc(NumberOfProcessorGroupsInternal());
	return GroupDesc;
}

int32 FWindowsPlatformMisc::NumberOfCoresIncludingHyperthreads()
{
	static int32 CoreCount = 0;
	if (CoreCount > 0)
	{
		return CoreCount;
	}

	FProcessorGroupDesc GroupDesc;
	uint32 NumaNodeCount = 0;
	uint32 NumCores = 0;
	uint32 LogicalProcessorCount = 0;
	QueryCpuInformation(GroupDesc, NumaNodeCount, NumCores, LogicalProcessorCount);

	bool bLimitsInitialized;
	int32 PhysicalCoreLimit;
	int32 LogicalCoreLimit;
	bool bSetPhysicalCountToLogicalCount;
	GetConfiguredCoreLimits(NumCores, LogicalProcessorCount, bLimitsInitialized, PhysicalCoreLimit,
		LogicalCoreLimit, bSetPhysicalCountToLogicalCount);

	CoreCount = LogicalProcessorCount;

	// Optionally limit number of threads (we don't necessarily scale super well with very high core counts)
	if (LogicalCoreLimit > 0)
	{
		CoreCount = FMath::Min(CoreCount, LogicalCoreLimit);
	}

	return CoreCount;
}

const TCHAR* FWindowsPlatformMisc::GetPlatformFeaturesModuleName()
{
	// If running a dedicated server then we use the default PlatformFeatures
	if (!IsRunningDedicatedServer())
	{
		static FString PlatformFeaturesName = "WindowsPlatformFeatures";
		static bool bIniChecked = false;
		if (!bIniChecked && !GEngineIni.IsEmpty())
		{
			GConfig->GetString(TEXT("PlatformFeatures"), TEXT("PlatformFeaturesModule"), PlatformFeaturesName, GEngineIni);
			bIniChecked = true;
		}

		bool bModuleExists = FModuleManager::Get().ModuleExists(*PlatformFeaturesName);
		if (bModuleExists && !PlatformFeaturesName.IsEmpty())
		{
			UE_LOGF(LogWindows, Log, "%ls enabled", *PlatformFeaturesName);
			return *PlatformFeaturesName;
		}
	}

	UE_LOGF(LogWindows, Log, "WindowsPlatformFeatures disabled or dedicated server build");
	return nullptr;
}

int32 FWindowsPlatformMisc::NumberOfWorkerThreadsToSpawn()
{	
	static int32 MaxServerWorkerThreads = 4;

	int32 MaxWorkerThreads = INT32_MAX;
	int32 NumberOfCores = FWindowsPlatformMisc::NumberOfCores();
	int32 NumberOfCoresIncludingHyperthreads = FWindowsPlatformMisc::NumberOfCoresIncludingHyperthreads();
	int32 NumberOfThreads = 0;

	if (NumberOfCoresIncludingHyperthreads > NumberOfCores)
	{
		NumberOfThreads = NumberOfCoresIncludingHyperthreads - 2;
	}
	else
	{
		NumberOfThreads = NumberOfCores - 1;
	}

	int32 MaxWorkerThreadsWanted = IsRunningDedicatedServer() ? MaxServerWorkerThreads : MaxWorkerThreads;
	// need to spawn at least one worker thread (see FTaskGraphImplementation)
	return FMath::Max(FMath::Min(NumberOfThreads, MaxWorkerThreadsWanted), 2);
}

bool FWindowsPlatformMisc::OsExecute(const TCHAR* CommandType, const TCHAR* Command, const TCHAR* CommandLine)
{
	HINSTANCE hApp = ShellExecute(NULL,
		CommandType,
		Command,
		CommandLine,
		NULL,
		SW_SHOWNORMAL);
	bool bSucceeded = hApp > (HINSTANCE)32;
	return bSucceeded;
}

struct FGetMainWindowHandleData
{
	HWND Handle;
	uint32 ProcessId;
};

int32 CALLBACK GetMainWindowHandleCallback(HWND Handle, LPARAM lParam)
{
	FGetMainWindowHandleData& Data = *(FGetMainWindowHandleData*)lParam;
	
	unsigned long ProcessId = 0;
	{
		::GetWindowThreadProcessId(Handle, &ProcessId);
	}
	
	if ((Data.ProcessId != ProcessId) || (::GetWindow(Handle, GW_OWNER) != (HWND)0) || !::IsWindowVisible(Handle))
	{
		return 1;
	}

	Data.Handle = Handle;

	return 0;
}

HWND FWindowsPlatformMisc::GetTopLevelWindowHandle(uint32 ProcessId)
{
	FGetMainWindowHandleData Data;
	{
		Data.Handle = 0;
		Data.ProcessId = ProcessId;
	}

	::EnumWindows(GetMainWindowHandleCallback, (LPARAM)&Data);

	return Data.Handle;
}

FORCENOINLINE void FWindowsPlatformMisc::RaiseException( uint32 ExceptionCode )
{
	/** This is the last place to gather memory stats before exception. */
	FGenericCrashContext::SetMemoryStats(FPlatformMemory::GetStats());

	::RaiseException( ExceptionCode, 0, 0, NULL );
}

bool FWindowsPlatformMisc::SetStoredValue(const FString& InStoreId, const FString& InSectionName, const FString& InKeyName, const FString& InValue)
{
	check(!InStoreId.IsEmpty());
	check(!InSectionName.IsEmpty());
	check(!InKeyName.IsEmpty());

	FString FullRegistryKey = FString("Software") / InStoreId / InSectionName;
	FullRegistryKey = FullRegistryKey.Replace(TEXT("/"), TEXT("\\")); // we use forward slashes, but the registry needs back slashes

	HKEY hKey;
	HRESULT Result = ::RegCreateKeyEx(HKEY_CURRENT_USER, *FullRegistryKey, 0, nullptr, REG_OPTION_NON_VOLATILE, KEY_WRITE, nullptr, &hKey, nullptr);
	if(Result == ERROR_SUCCESS)
	{
		Result = ::RegSetValueEx(hKey, *InKeyName, 0, REG_SZ, (const BYTE*)*InValue, (InValue.Len() + 1) * sizeof(TCHAR));
		::RegCloseKey(hKey);
	}
	
	if(Result != ERROR_SUCCESS)
	{
		TCHAR ErrorBuffer[1024];
		::FormatMessageW(FORMAT_MESSAGE_FROM_SYSTEM, nullptr, Result, 0, ErrorBuffer, 1024, nullptr);
		GWarn->Logf(TEXT("FWindowsPlatformMisc::SetStoredValue: ERROR: Could not store value for '%s'. Error Code %u: %s"), *InKeyName, Result, ErrorBuffer);
		return false;
	}

	return true;
}

bool FWindowsPlatformMisc::GetStoredValue(const FString& InStoreId, const FString& InSectionName, const FString& InKeyName, FString& OutValue)
{
	check(!InStoreId.IsEmpty());
	check(!InSectionName.IsEmpty());
	check(!InKeyName.IsEmpty());

	FString FullRegistryKey = FString("Software") / InStoreId / InSectionName;
	FullRegistryKey = FullRegistryKey.Replace(TEXT("/"), TEXT("\\")); // we use forward slashes, but the registry needs back slashes

	return QueryRegKey(HKEY_CURRENT_USER, *FullRegistryKey, *InKeyName, OutValue);
}

bool FWindowsPlatformMisc::DeleteStoredValue(const FString& InStoreId, const FString& InSectionName, const FString& InKeyName)
{
	// Deletes values in reg keys and also deletes the owning key if it becomes empty

	check(!InStoreId.IsEmpty());
	check(!InSectionName.IsEmpty());
	check(!InKeyName.IsEmpty());

	FString FullRegistryKey = FString("Software") / InStoreId / InSectionName;
	FullRegistryKey = FullRegistryKey.Replace(TEXT("/"), TEXT("\\")); // we use forward slashes, but the registry needs back slashes

	HKEY hKey;
	HRESULT Result = ::RegOpenKeyEx(HKEY_CURRENT_USER, *FullRegistryKey, 0, KEY_WRITE | KEY_READ, &hKey);
	if (Result == ERROR_SUCCESS)
	{
		Result = ::RegDeleteValue(hKey, *InKeyName);

		// Query for sub-keys in the open key
		TCHAR CheckKeyName[256];
		::DWORD CheckKeyNameLength = sizeof(CheckKeyName) / sizeof(CheckKeyName[0]);
		HRESULT EnumResult = RegEnumKeyEx(hKey, 0, CheckKeyName, &CheckKeyNameLength, NULL, NULL, NULL, NULL);
		bool bZeroSubKeys = EnumResult != ERROR_SUCCESS;

		// Query for a remaining value in the open key
		wchar_t CheckValueName[256];
		::DWORD CheckValueNameLength = sizeof(CheckValueName) / sizeof(CheckValueName[0]);
		EnumResult = RegEnumValue(hKey, 0, CheckValueName, &CheckValueNameLength, NULL, NULL, NULL, NULL);
		bool bZeroValues = EnumResult != ERROR_SUCCESS;

		::RegCloseKey(hKey);

		if (bZeroSubKeys && bZeroValues)
		{
			// No more values - delete the section
			::RegDeleteKey(HKEY_CURRENT_USER, *FullRegistryKey);
		}
	}

	return Result == ERROR_SUCCESS;
}

bool FWindowsPlatformMisc::DeleteStoredSection(const FString& InStoreId, const FString& InSectionName)
{
	check(!InStoreId.IsEmpty());
	check(!InSectionName.IsEmpty());

	FString FullRegistryKey = FString("Software") / InStoreId / InSectionName;
	FullRegistryKey = FullRegistryKey.Replace(TEXT("/"), TEXT("\\")); // we use forward slashes, but the registry needs back slashes

	return ::RegDeleteTree(HKEY_CURRENT_USER, *FullRegistryKey) == ERROR_SUCCESS;
}

FString FWindowsPlatformMisc::GetDefaultLanguage()
{
	// Only use GetUserPreferredUILanguages on Windows 8+ as older versions didn't always have language packs available
	if (FPlatformMisc::VerifyWindowsVersion(6, 2))
	{
		ULONG NumLanguages = 0;
		ULONG LangBufferSize = 0;
		if (::GetUserPreferredUILanguages(MUI_LANGUAGE_NAME, &NumLanguages, nullptr, &LangBufferSize))
		{
			TArray<WCHAR> LangBuffer;
			LangBuffer.SetNumZeroed(LangBufferSize);
		
			if (::GetUserPreferredUILanguages(MUI_LANGUAGE_NAME, &NumLanguages, LangBuffer.GetData(), &LangBufferSize))
			{
				// GetUserPreferredUILanguages returns a list where each item is null terminated, so this produces a string containing only the first item
				return FString(LangBuffer.GetData());
			}
		}
	}
	
	return GetDefaultLocale();
}

FString FWindowsPlatformMisc::GetDefaultLocale()
{
	WCHAR LocaleName[LOCALE_NAME_MAX_LENGTH];
	if (::GetUserDefaultLocaleName(LocaleName, LOCALE_NAME_MAX_LENGTH))
	{
		return FString(LocaleName);
	}

	return FGenericPlatformMisc::GetDefaultLocale();
}

uint32 FWindowsPlatformMisc::GetLastError()
{
	return (uint32)::GetLastError();
}

void FWindowsPlatformMisc::SetLastError(uint32 ErrorCode)
{
	::SetLastError((DWORD)ErrorCode);
}

bool FWindowsPlatformMisc::CoInitialize(ECOMModel Model)
{
	HRESULT hr = ::CoInitializeEx(NULL, (Model == ECOMModel::Singlethreaded) ? COINIT_APARTMENTTHREADED : COINIT_MULTITHREADED);
	return hr == S_OK || hr == S_FALSE;
}

void FWindowsPlatformMisc::CoUninitialize()
{
	::CoUninitialize();
}

#if !UE_BUILD_SHIPPING
static TCHAR GErrorRemoteDebugPromptMessage[MAX_SPRINTF];

void FWindowsPlatformMisc::PromptForRemoteDebugging(bool bIsEnsure)
{
	if (bShouldPromptForRemoteDebugging)
	{
		if (bIsEnsure && !bPromptForRemoteDebugOnEnsure)
		{
			// Don't prompt on ensures unless overridden
			return;
		}

		if (FApp::IsUnattended())
		{
			// Do not ask if there is no one to show a message
			return;
		}

		if (GIsCriticalError && !GIsGuarded)
		{
			// A fatal error occurred.
			// We have not ability to debug, this does not make sense to ask.
			return;
		}

		// Upload locally compiled files for remote debugging
		FPlatformStackWalk::UploadLocalSymbols();

		FCString::Sprintf(GErrorRemoteDebugPromptMessage, 
			TEXT("Have a programmer remote debug this crash?\n"
			     "Hit NO to exit and submit error report as normal.\n"
			     "Otherwise, contact a programmer for remote debugging,\n"
			     "giving them the changelist number below.\n"
			     "Once they have confirmed they are connected to the machine,\n"
			     "hit YES to allow them to debug the crash.\n"
			     "[Changelist = %d]"),
			FEngineVersion::Current().GetChangelist());
		FSlowHeartBeatScope SuspendHeartBeat;
		if (MessageBox(0, GErrorRemoteDebugPromptMessage, TEXT("CRASHED"), MB_YESNO|MB_SYSTEMMODAL) == IDYES)
		{
			::DebugBreak();
		}
	}
}
#endif	//#if !UE_BUILD_SHIPPING

/**
* Class that caches __cpuid queried data.
*/
class FCPUIDQueriedData
{
public:
	FCPUIDQueriedData()
		: bHasCPUIDInstruction(CheckForCPUIDInstruction()), Vendor(), CPUInfo(0), CacheLineSize(PLATFORM_CACHE_LINE_SIZE)
	{
		bHasTimedPauseInstruction = false;
		if(bHasCPUIDInstruction)
		{
			GetCPUVendor(Vendor);
			GetCPUBrand(Brand);
			int Info[4];
			QueryCPUInfo(Info);
			CPUInfo = Info[0];
			CPUInfo2 = Info[2];
			CacheLineSize = QueryCacheLineSize();
			bHasTimedPauseInstruction = CheckForTimedPauseInstruction();
			UE_CLOGF(bHasTimedPauseInstruction, LogWindows, Log, "Enabling Tpause support");
		}
	}

	/** 
	 * Checks if this CPU supports __cpuid instruction.
	 *
	 * @returns True if this CPU supports __cpuid instruction. False otherwise.
	 */
	static bool HasCPUIDInstruction()
	{
		return CPUIDStaticCache.bHasCPUIDInstruction;
	}

	/**
	 * Checks if this CPU supports tpause instruction.
	 *
	 * @returns True if this CPU supports tpause instruction. False otherwise.
	 */
	static bool HasTimedPauseInstruction()
	{
		return CPUIDStaticCache.bHasTimedPauseInstruction;
	}

	/**
	 * Gets pre-cached CPU vendor name.
	 *
	 * @returns CPU vendor name.
	 */
	static const ANSICHAR (&GetVendor())[12 + 1]
	{
		return CPUIDStaticCache.Vendor;
	}

	/**
	* Gets pre-cached CPU brand string.
	*
	* @returns CPU brand string.
	*/
	static const ANSICHAR (&GetBrand())[0x40]
	{
		return CPUIDStaticCache.Brand;
	}

	/**
	 * Gets __cpuid CPU info.
	 *
	 * @returns CPU info unsigned int queried using __cpuid.
	 */
	static uint32 GetCPUInfo()
	{
		return CPUIDStaticCache.CPUInfo;
	}

	/**
	* Gets __cpuid CPU info.
	*
	* @returns CPU info unsigned int queried using __cpuid.
	*/
	static uint32 GetCPUInfo2()
	{
		return CPUIDStaticCache.CPUInfo2;
	}

private:

#if !PLATFORM_CPU_ARM_FAMILY
	/**
	 * Checks if __cpuid instruction is present on current machine.
	 *
	 * @returns True if this CPU supports __cpuid instruction. False otherwise.
	 */
	static bool CheckForCPUIDInstruction()
	{
		// all x86 64-bit CPUs support CPUID, no check required
#if PLATFORM_HAS_CPUID
		return true;
#else
	#if PLATFORM_SEH_EXCEPTIONS_DISABLED
		return false;
	#else
		__try
		{
			int Args[4];
			__cpuid(Args, 0);
		}
		__except (EXCEPTION_EXECUTE_HANDLER)
		{
			return false;
		}
		return true;
	#endif
#endif
	}

	static int FilterInvalidOpcode(DWORD ExceptionCode, struct _EXCEPTION_POINTERS* ExceptionInformation)
	{
		if (ExceptionCode == STATUS_ILLEGAL_INSTRUCTION || ExceptionCode == STATUS_PRIVILEGED_INSTRUCTION)
		{
			return EXCEPTION_EXECUTE_HANDLER;
		}
		else
		{
			return EXCEPTION_CONTINUE_SEARCH;
		}
	}

	/**
	 * Checks if tpause instruction is present on current machine.
	 *
	 * @returns True if this CPU supports tpause instruction. False otherwise.
	 */
	static bool CheckForTimedPauseInstruction()
	{
#if PLATFORM_SEH_EXCEPTIONS_DISABLED
		return false;
#else
		bool bSupportsTpause = false;
		int CPUInfo[4];
		UE_CPUID(CPUInfo, 0);

		if (CPUInfo[0] >= 7)
		{
			int CPUExtendedInfo[4];
			__cpuidex(CPUExtendedInfo, 7, 0);

			if ((CPUExtendedInfo[2] & (1 << 5)) != 0)
			{
				// WAITPKG is supported
				__try
				{
					unsigned long long tsc = __rdtsc();
					_tpause(0, tsc + 1024);
					// TPAUSE is supported
					bSupportsTpause = true;
				}
				__except (FilterInvalidOpcode(GetExceptionCode(), GetExceptionInformation()))
				{
					bSupportsTpause = false;
				}
			}
		}

		return bSupportsTpause;
#endif
	}

	/**
	 * Queries Vendor name using __cpuid instruction.
	 *
	 * @returns CPU vendor name.
	 */
	static void GetCPUVendor(ANSICHAR (&OutBuffer)[12 + 1])
	{
		union
		{
			char Buffer[12 + 1];
			struct
			{
				int dw0;
				int dw1;
				int dw2;
			} Dw;
		} VendorResult;

		int Args[4];
		UE_CPUID(Args, 0);

		VendorResult.Dw.dw0 = Args[1];
		VendorResult.Dw.dw1 = Args[3];
		VendorResult.Dw.dw2 = Args[2];
		VendorResult.Buffer[12] = 0;

		FMemory::Memcpy(OutBuffer, VendorResult.Buffer);
	}

	/**
	 * Queries brand string using __cpuid instruction.
	 *
	 * @returns CPU brand string.
	 */
	static void GetCPUBrand(ANSICHAR (&OutBrandString)[0x40])
	{
		// @see for more information http://msdn.microsoft.com/en-us/library/vstudio/hskdteyh(v=vs.100).aspx
		ANSICHAR BrandString[0x40] = {0};
		int32 CPUInfo[4] = {-1};
		const SIZE_T CPUInfoSize = sizeof( CPUInfo );

		UE_CPUID(CPUInfo, 0x80000000);
		const uint32 MaxExtIDs = CPUInfo[0];

		if( MaxExtIDs >= 0x80000004 )
		{
			const uint32 FirstBrandString = 0x80000002;
			const uint32 NumBrandStrings = 3;
			for( uint32 Index = 0; Index < NumBrandStrings; Index++ )
			{
				UE_CPUID(CPUInfo, FirstBrandString + Index);
				FPlatformMemory::Memcpy( BrandString + CPUInfoSize * Index, CPUInfo, CPUInfoSize );
			}
		}

		FMemory::Memcpy(OutBrandString, BrandString);
	}

	/**
	 * Queries CPU info using __cpuid instruction.
	 *
	 * @returns CPU info unsigned int queried using __cpuid.
	 */
	static void QueryCPUInfo(int Args[4])
	{
		UE_CPUID(Args, 1);
	}

	/**
	 * Queries cache line size using __cpuid instruction.
	 *
	 * @returns Cache line size.
	 */
	static int32 QueryCacheLineSize()
	{
		int32 Result = 1;

		int Args[4];
		UE_CPUID(Args, 0x80000006);

		Result = Args[2] & 0xFF;
		check(Result && !(Result & (Result - 1))); // assumed to be a power of two

		return Result;
	}
#else
	static bool CheckForCPUIDInstruction() { return false; }
	static bool CheckForTimedPauseInstruction() { return false; }
	static void GetCPUVendor(ANSICHAR(&OutBuffer)[12 + 1]) {}
	static void GetCPUBrand(ANSICHAR(&OutBrandString)[0x40]) {}
	static void QueryCPUInfo(int Args[4]) {}
	static int32 QueryCacheLineSize() { return PLATFORM_CACHE_LINE_SIZE; }
#endif

	/** Static field with pre-cached __cpuid data. */
	static FCPUIDQueriedData CPUIDStaticCache;

	/** If machine has CPUID instruction. */
	bool bHasCPUIDInstruction;

	/** If machine has timed pause instruction. */
	bool bHasTimedPauseInstruction;

	/** Vendor of the CPU. */
	ANSICHAR Vendor[12 + 1];

	/** CPU brand. */
	ANSICHAR Brand[0x40];

	/** CPU info from __cpuid. */
	uint32 CPUInfo;
	uint32 CPUInfo2;

	/** CPU cache line size. */
	int32 CacheLineSize;
};

/** Static initialization of data to pre-cache __cpuid queries. */
FCPUIDQueriedData FCPUIDQueriedData::CPUIDStaticCache;

bool FWindowsPlatformMisc::HasCPUIDInstruction()
{
	return FCPUIDQueriedData::HasCPUIDInstruction();
}

FString FWindowsPlatformMisc::GetCPUVendor()
{
	return FCPUIDQueriedData::GetVendor();
}

FString FWindowsPlatformMisc::GetCPUBrand()
{
	return FCPUIDQueriedData::GetBrand();
}

uint32 FWindowsPlatformMisc::GetCPUModelID()
{
	uint32 CpuID = FCPUIDQueriedData::GetCPUInfo();
	uint32 SteppingID = CpuID & 0x0F;
	uint32 ModelID = (CpuID >> 4) & 0x0F;
	uint32 FamilyID = (CpuID >> 8) & 0x0F;

	if (FamilyID == 0x06 || FamilyID == 0x0F)
	{
		uint32 ExtendedModelID = (CpuID >> 16) & 0x0F;
		ModelID = (ExtendedModelID << 4) + ModelID;
	}

	if (FamilyID == 0x0F)
	{
		uint32 ExtendedFamilyID = (CpuID >> 20) & 0xFF;
		FamilyID = FamilyID + ExtendedFamilyID;
	}

	// Repack Family (12 bits), Model (8 bits), Stepping (4 bits) into a single 32-bit integer.
	return (FamilyID << 12) | (ModelID << 4) | SteppingID;
}

uint32 FWindowsPlatformMisc::GetCPUMicrocodeRevision()
{
	// The microcode revision number is only available in a model-specific register, so we can't read it from userland. However, Windows
	// seems to store the value in the registry at boot time, so we can try to get it from there.

	HKEY Key = 0;
	if (RegOpenKeyEx(HKEY_LOCAL_MACHINE, TEXT("HARDWARE\\DESCRIPTION\\System\\CentralProcessor\\0"), 0, KEY_READ, &Key) != ERROR_SUCCESS)
	{
		return 0;
	}

	uint32 MicrocodeVersion = 0;

	uint32 MicrocodeVersionBuf[2];
	::DWORD Size = 0, ValueType = 0;
	if (RegQueryValueEx(Key, TEXT("Update Revision"), NULL, &ValueType, NULL, &Size) == ERROR_SUCCESS && ValueType == REG_BINARY && Size <= sizeof(MicrocodeVersionBuf))
	{
		if (RegQueryValueEx(Key, TEXT("Update Revision"), NULL, NULL, (LPBYTE)MicrocodeVersionBuf, &Size) == ERROR_SUCCESS)
		{
			if (Size == 8)
			{
				// On older Windows versions the registry key contains the full 64-bit value of the IA32_BIOS_SIGN_ID MSR. In that case,
				// the microcode revision number is in the high 32 bits.
				MicrocodeVersion = MicrocodeVersionBuf[1];
			}
			else if (Size == 4)
			{
				// In most cases, Windows already shifts the value and stores only the 32-bit microcode revision number.
				MicrocodeVersion = MicrocodeVersionBuf[0];
			}
		}
	}

	RegCloseKey(Key);
	return MicrocodeVersion;
}

#include "Windows/AllowWindowsPlatformTypes.h"
#include <tbs.h>
bool FWindowsPlatformMisc::HasTPM2Support()
{
	auto HasTPM2SupportInternal = []() -> bool
	{
#if (NTDDI_VERSION >= NTDDI_WIN8)
#pragma comment(lib, "tbs.lib")
		TPM_DEVICE_INFO TpmInfo;
		FMemory::Memzero(TpmInfo);
		if (Tbsi_GetDeviceInfo(sizeof(TpmInfo), &TpmInfo) == TBS_SUCCESS)
		{
			return TpmInfo.tpmVersion == TPM_VERSION_20;
		}
#else
		// Until _WIN32_WINNT is set to _WIN32_WINNT_WIN8 we will need to check TPM status dynamically
		if (void* hTbs = FPlatformProcess::GetDllHandle(TEXT("tbs.dll")))
		{
			struct TPM_DEVICE_INFO
			{
				UINT32 structVersion;
				UINT32 tpmVersion; // 1.2 = 1, 2.0 = 2
				UINT32 tpmInterfaceType;
				UINT32 tpmImpRevision;
			};
			// https://learn.microsoft.com/en-us/windows/win32/api/tbs/nf-tbs-tbsi_getdeviceinfo
			typedef TBS_RESULT (WINAPI *pfnTbsiGetDeviceInfo)(UINT32 Size, PVOID Info);
			if (const pfnTbsiGetDeviceInfo Tbsi_GetDeviceInfo = reinterpret_cast<pfnTbsiGetDeviceInfo>(FPlatformProcess::GetDllExport(hTbs, TEXT("Tbsi_GetDeviceInfo"))))
			{
				TPM_DEVICE_INFO TpmInfo;
				FMemory::Memzero(TpmInfo);
				if (Tbsi_GetDeviceInfo(sizeof(TpmInfo), &TpmInfo) == TBS_SUCCESS)
				{
					FPlatformProcess::FreeDllHandle(hTbs);
					return TpmInfo.tpmVersion == 2;
				}
				UE_LOGF(LogWindows, Warning, "Failed to retrieve TPM device information");
			}
			else
			{
				UE_LOGF(LogWindows, Warning, "Failed to load proc address for Tbsi_GetDeviceInfo");
			}
			FPlatformProcess::FreeDllHandle(hTbs);
		}
		else
		{
			UE_LOGF(LogWindows, Warning, "Failed to load tbs.dll");
		}
#endif
		return false;
	};
	static bool bSupportsTpm2 = HasTPM2SupportInternal();
	return bSupportsTpm2;
}

#include "winternl.h"
#include "ntstatus.h"
bool FWindowsPlatformMisc::HasSecureBootSupport()
{
	auto HasSecureBootSupportInternal = []() -> bool
	{
		if (void* hNtDll = FPlatformProcess::GetDllHandle(TEXT("ntdll.dll")))
		{
			// Using an undocumented SYSTEM_INFORMATION_CLASS type SYSTEM_SECUREBOOT_INFORMATION
			static SYSTEM_INFORMATION_CLASS SecureBootInformationClassId = static_cast<SYSTEM_INFORMATION_CLASS>(0x91);
			struct SYSTEM_SECUREBOOT_INFORMATION
			{
				bool bSecureBootEnabled;
				bool bSecureBootCapable;
			};
			// https://learn.microsoft.com/en-us/windows/win32/api/winternl/nf-winternl-ntquerysysteminformation
			typedef NTSTATUS (*pfnNtQuerySystemInformation)(SYSTEM_INFORMATION_CLASS SystemInformationClass, PVOID SystemInformation, ULONG SystemInformationLength, PULONG ReturnLength);
			if (const pfnNtQuerySystemInformation fnNtQuerySystemInformation = reinterpret_cast<pfnNtQuerySystemInformation>(FPlatformProcess::GetDllExport(hNtDll, TEXT("NtQuerySystemInformation"))))
			{
				SYSTEM_SECUREBOOT_INFORMATION SecureBootInfo;
				FMemory::Memzero(SecureBootInfo);
				if (fnNtQuerySystemInformation(SecureBootInformationClassId, &SecureBootInfo, sizeof(SecureBootInfo), nullptr) == STATUS_SUCCESS)
				{
					FPlatformProcess::FreeDllHandle(hNtDll);
					return SecureBootInfo.bSecureBootCapable;
				}
				UE_LOGF(LogWindows, Warning, "Failed to retrieve secure boot information");
			}
			else
			{
				UE_LOGF(LogWindows, Warning, "Failed to load proc address for NtQuerySystemInformation");
			}
			FPlatformProcess::FreeDllHandle(hNtDll);
		}
		else
		{
			UE_LOGF(LogWindows, Warning, "Failed to load ntdll.dll");
		}
		return false;
	};
	static bool bHasSecureBootSupport = HasSecureBootSupportInternal();
	return bHasSecureBootSupport;
}
#include "Windows/HideWindowsPlatformTypes.h"

bool FWindowsPlatformMisc::SupportsWindows11()
{
	static struct FWindows11SupportCheck
	{
		bool bSupported;
		
		FWindows11SupportCheck()
		{
			// https://www.microsoft.com/en-us/windows/windows-11-specifications?r=1
			// Note that this check is mostly complete, but does not check for:
			// - Whether the CPU is on Microsoft's approved processor list https://aka.ms/CPUlist
			// - DX12 Support
			bSupported = HasTPM2Support();
			bSupported &= HasSecureBootSupport();
			bSupported &= NumberOfCores() >= 2;
			bSupported &= FPlatformMemory::GetPhysicalGBRam() >= 4;
		}
	} Win11Check;
	
	return Win11Check.bSupported;
}

#include "Windows/AllowWindowsPlatformTypes.h"
FString FWindowsPlatformMisc::GetPrimaryGPUBrand()
{
	static FString PrimaryGPUBrand;
	if( PrimaryGPUBrand.IsEmpty() )
	{
		// Find primary display adapter and get the device name.
		PrimaryGPUBrand = FGenericPlatformMisc::GetPrimaryGPUBrand();

		DISPLAY_DEVICE DisplayDevice;
		DisplayDevice.cb = sizeof( DisplayDevice );
		DWORD DeviceIndex = 0;

		while( EnumDisplayDevices( 0, DeviceIndex, &DisplayDevice, 0 ) )
		{
			if( (DisplayDevice.StateFlags & (DISPLAY_DEVICE_ATTACHED_TO_DESKTOP | DISPLAY_DEVICE_PRIMARY_DEVICE)) > 0 )
			{
				PrimaryGPUBrand = DisplayDevice.DeviceString;
				break;
			}

			FMemory::Memzero( DisplayDevice );
			DisplayDevice.cb = sizeof( DisplayDevice );
			DeviceIndex++;
		}
	}

	return PrimaryGPUBrand;
}

#define USE_SP_ALTPLATFORM_INFO_V1 0
#define USE_SP_ALTPLATFORM_INFO_V3 1
#define USE_SP_DRVINFO_DATA_V1 0
#define USE_SP_BACKUP_QUEUE_PARAMS_V1 0
#define USE_SP_INF_SIGNER_INFO_V1 0
#include <SetupAPI.h>
#include <initguid.h>
#include <devguid.h>
#include <devpkey.h>
#undef USE_SP_ALTPLATFORM_INFO_V1
#undef USE_SP_ALTPLATFORM_INFO_V3
#undef USE_SP_DRVINFO_DATA_V1
#undef USE_SP_BACKUP_QUEUE_PARAMS_V1
#undef USE_SP_INF_SIGNER_INFO_V1

static void GetVideoDriverDetailsFromSetup(const FString& DeviceName, bool bVerbose, FGPUDriverInfo& Out)
{
	
	HDEVINFO hDevInfo = SetupDiGetClassDevs(&GUID_DEVCLASS_DISPLAY, NULL, NULL, DIGCF_PRESENT);

	FString RegistryKey;
	
	if (hDevInfo != INVALID_HANDLE_VALUE)
	{
		DWORD DataType = 0;
		
		const uint32 BufferSize = 512;
		TCHAR Buffer[BufferSize + 1] = { 0 };
		
		SP_DEVINFO_DATA DeviceInfoData;
		DeviceInfoData.cbSize = sizeof(SP_DEVINFO_DATA);

		bool bFound = false;
		for (int32 Idx = 0; SetupDiEnumDeviceInfo(hDevInfo, Idx, &DeviceInfoData); Idx++)
		{
			// Get the device description, check if it matches the queried device name
			if (SetupDiGetDeviceProperty(hDevInfo, &DeviceInfoData, &DEVPKEY_Device_DriverDesc, &DataType,
				(PBYTE)Buffer, sizeof(Buffer), nullptr, 0))
			{
				if (DeviceName.Compare(Buffer) == 0)
				{
					Out.DeviceDescription = Buffer;
					bFound = true;
					ZeroMemory(Buffer, sizeof(Buffer));

					// Retrieve the registry key for this device for 3rd party data
					if (SetupDiGetDeviceProperty(hDevInfo, &DeviceInfoData, &DEVPKEY_Device_Driver, &DataType,
						(PBYTE)Buffer, sizeof(Buffer), nullptr, 0))
					{
						RegistryKey = Buffer;
						ZeroMemory(Buffer, sizeof(Buffer));
					}
					else
					{
						UE_CLOGF(bVerbose, LogWindows, Log, "Failed to retrieve driver registry key for device %d", Idx);
					}
					
					break;
				}
				ZeroMemory(Buffer, sizeof(Buffer));
			}
			else
			{
				UE_CLOGF(bVerbose, LogWindows, Log, "Failed to retrieve driver description for device %d", Idx);
			}
		}

		if (bFound)
		{
			// Get the provider name
			if (SetupDiGetDeviceProperty(hDevInfo, &DeviceInfoData, &DEVPKEY_Device_DriverProvider, &DataType,
				(PBYTE)Buffer, sizeof(Buffer), nullptr, 0))
			{
				Out.ProviderName = Buffer;
				ZeroMemory(Buffer, sizeof(Buffer));
			}
			else
			{
				UE_CLOGF(bVerbose, LogWindows, Log, "Failed to find provider name");
			}
			// Get the internal driver version
			if (SetupDiGetDeviceProperty(hDevInfo, &DeviceInfoData, &DEVPKEY_Device_DriverVersion, &DataType,
				(PBYTE)Buffer, sizeof(Buffer), nullptr, 0))
			{
				Out.InternalDriverVersion = Buffer;
				ZeroMemory(Buffer, sizeof(Buffer));
			}
			else
			{
				UE_CLOGF(bVerbose, LogWindows, Log, "Failed to find internal driver version");
			}
			// Get the driver date
			FILETIME FileTime;
			if (SetupDiGetDeviceProperty(hDevInfo, &DeviceInfoData, &DEVPKEY_Device_DriverDate, &DataType,
				(PBYTE)&FileTime, sizeof(FILETIME), nullptr, 0))
			{
				SYSTEMTIME SystemTime;
				FileTimeToSystemTime(&FileTime, &SystemTime);
				Out.DriverDate = FString::Printf(TEXT("%d-%d-%d"), SystemTime.wMonth, SystemTime.wDay, SystemTime.wYear);
			}
			else
			{
				UE_CLOGF(bVerbose, LogWindows, Log, "Failed to find driver date");
			}
		}
		else
		{
			UE_CLOGF(bVerbose, LogWindows, Log, "Unable to find requested device '%ls' using Setup API.", *DeviceName);
		}
		
		SetupDiDestroyDeviceInfoList(hDevInfo);
	}
	else
	{
		UE_CLOGF(bVerbose, LogWindows, Log, "Failed to initialize Setup API");
	}

	if (!Out.ProviderName.IsEmpty())
	{
		if (Out.ProviderName.Contains(TEXT("NVIDIA")))
		{
			Out.SetNVIDIA();
		}
		else if (Out.ProviderName.Contains(TEXT("Advanced Micro Devices")))
		{
			Out.SetAMD();
		}
		else if (Out.ProviderName.Contains(TEXT("Intel")))	// usually TEXT("Intel Corporation")
		{
			Out.SetIntel();
		}
		else if (Out.ProviderName.Contains(TEXT("Qualcomm")))
		{
			Out.SetQualcomm();
		}
	}

	Out.UserDriverVersion = Out.InternalDriverVersion;

	if(Out.IsNVIDIA())
	{
		Out.UserDriverVersion = Out.GetNVIDIAUnifiedVersion(Out.InternalDriverVersion);
	}
	else if(Out.IsAMD() && !RegistryKey.IsEmpty())
	{
		// Get the AMD specific information directly from the registry
		// AMD AGS could be used instead, but retrieving the radeon software version cannot occur after a D3D Device
		// has been created, and this function could be called at any time
		
		const FString Key = FString::Printf(TEXT("SYSTEM\\CurrentControlSet\\Control\\Class\\%s"), *RegistryKey);
		
		if(FWindowsPlatformMisc::QueryRegKey(HKEY_LOCAL_MACHINE, *Key, TEXT("Catalyst_Version"), Out.UserDriverVersion))
		{
			Out.UserDriverVersion = FString("Catalyst ") + Out.UserDriverVersion;
		}

		FString Edition;
		if(FWindowsPlatformMisc::QueryRegKey(HKEY_LOCAL_MACHINE, *Key, TEXT("RadeonSoftwareEdition"), Edition))
		{
			FString Version;
			if(FWindowsPlatformMisc::QueryRegKey(HKEY_LOCAL_MACHINE, *Key, TEXT("RadeonSoftwareVersion"), Version))
			{
				// e.g. TEXT("Crimson 15.12") or TEXT("Catalyst 14.1")
				Out.UserDriverVersion = Edition + TEXT(" ") + Version;
			}
		}
	}

}

static void GetVideoDriverDetails(const FString& Key, FGPUDriverInfo& Out)
{
	// https://msdn.microsoft.com/en-us/library/windows/hardware/ff569240(v=vs.85).aspx

	const TCHAR* DeviceDescriptionValueName = TEXT("Device Description");

	bool bDevice = FWindowsPlatformMisc::QueryRegKey(HKEY_LOCAL_MACHINE, *Key, DeviceDescriptionValueName, Out.DeviceDescription); // AMD and NVIDIA
	if (!bDevice)
	{
		bDevice = FWindowsPlatformMisc::QueryRegKey(HKEY_LOCAL_MACHINE, *Key, TEXT("DriverDesc"), Out.DeviceDescription);
	}

	if (!bDevice)
	{
		// in the case where this failed we also have:
		//  "DriverDesc"="NVIDIA GeForce GTX 670"
		
		// e.g. "GeForce GTX 680" (no NVIDIA prefix so no good for string comparison with DX)
		//	FWindowsPlatformMisc::QueryRegKey(HKEY_LOCAL_MACHINE, *Key, TEXT("HardwareInformation.AdapterString"), Out.DeviceDescription); // AMD and NVIDIA

		// Try again in Settings subfolder
		const FString SettingsSubKey = Key + TEXT("\\Settings");
		bDevice = FWindowsPlatformMisc::QueryRegKey(HKEY_LOCAL_MACHINE, *SettingsSubKey, DeviceDescriptionValueName, Out.DeviceDescription); // AMD and NVIDIA

		if (!bDevice)
		{
			// Neither root nor Settings subfolder contained a "Device Description" value so this is probably not a device
			Out = FGPUDriverInfo();
			return;
		}
	}

	// some key/value pairs explained: http://www.helpdoc-online.com/SCDMS01EN1A330P306~Windows-NT-Workstation-3.51-Resource-Kit-Help-en~Video-Device-Driver-Entries.htm

	FWindowsPlatformMisc::QueryRegKey(HKEY_LOCAL_MACHINE, *Key, TEXT("ProviderName"), Out.ProviderName);

	if (!Out.ProviderName.IsEmpty())
	{
		if (Out.ProviderName.Find(TEXT("NVIDIA")) != INDEX_NONE)
		{
			Out.SetNVIDIA();
		}
		else if (Out.ProviderName.Find(TEXT("Advanced Micro Devices")) != INDEX_NONE)
		{
			Out.SetAMD();
		}
		else if (Out.ProviderName.Find(TEXT("Intel")) != INDEX_NONE)	// usually TEXT("Intel Corporation")
		{
			Out.SetIntel();
		}
		else if (Out.ProviderName.Find(TEXT("Qualcomm")) != INDEX_NONE)
		{
			Out.SetQualcomm();
		}
	}

	// technical driver version, AMD and NVIDIA
	FWindowsPlatformMisc::QueryRegKey(HKEY_LOCAL_MACHINE, *Key, TEXT("DriverVersion"), Out.InternalDriverVersion);

	Out.UserDriverVersion = Out.InternalDriverVersion;

	if(Out.IsNVIDIA())
	{
		Out.UserDriverVersion = Out.GetNVIDIAUnifiedVersion(Out.InternalDriverVersion);
	}
	else if(Out.IsAMD())
	{
		if(FWindowsPlatformMisc::QueryRegKey(HKEY_LOCAL_MACHINE, *Key, TEXT("Catalyst_Version"), Out.UserDriverVersion))
		{
			Out.UserDriverVersion = FString("Catalyst ") + Out.UserDriverVersion;
		}

		FString Edition;
		if(FWindowsPlatformMisc::QueryRegKey(HKEY_LOCAL_MACHINE, *Key, TEXT("RadeonSoftwareEdition"), Edition))
		{
			FString Version;
			if(FWindowsPlatformMisc::QueryRegKey(HKEY_LOCAL_MACHINE, *Key, TEXT("RadeonSoftwareVersion"), Version))
			{
				// e.g. TEXT("Crimson 15.12") or TEXT("Catalyst 14.1")
				Out.UserDriverVersion = Edition + TEXT(" ") + Version;
			}
		}
	}

	// AMD and NVIDIA
	FWindowsPlatformMisc::QueryRegKey(HKEY_LOCAL_MACHINE, *Key, TEXT("DriverDate"), Out.DriverDate);
}

static BOOL CALLBACK MonitorEnumProc(HMONITOR Monitor, HDC MonitorDC, LPRECT Rect, LPARAM UserData)
{
	int* NumMonitors = (int*)UserData;
	*NumMonitors += 1;

	MONITORINFOEX MonitorInfoEx;
	MonitorInfoEx.cbSize = sizeof(MonitorInfoEx);
	GetMonitorInfo(Monitor, &MonitorInfoEx);

	UE_LOGF(LogWindows, Log, "    resolution: %dx%d, work area: (%d, %d) -> (%d, %d), device: '%ls'%ls",
		MonitorInfoEx.rcMonitor.right - MonitorInfoEx.rcMonitor.left, MonitorInfoEx.rcMonitor.bottom - MonitorInfoEx.rcMonitor.top,
		MonitorInfoEx.rcWork.left, MonitorInfoEx.rcWork.top, MonitorInfoEx.rcWork.right, MonitorInfoEx.rcWork.bottom,
		MonitorInfoEx.szDevice,
		MonitorInfoEx.dwFlags & MONITORINFOF_PRIMARY ? TEXT(" [PRIMARY]") : TEXT("")
	);

	return TRUE;
}

FGPUDriverInfo FWindowsPlatformMisc::GetGPUDriverInfo(const FString& DeviceDescription, bool bVerbose)
{
	if (bVerbose)
	{
		// Also report monitor information here, for lack of a better place.
		UE_LOGF(LogWindows, Log, "Attached monitors:");
		int NumMonitors = 0;
		EnumDisplayMonitors(nullptr, nullptr, MonitorEnumProc, (LPARAM)&NumMonitors);
		UE_LOGF(LogWindows, Log, "Found %d attached monitors.", NumMonitors);
	}

	// to distinguish failed GetGPUDriverInfo() from call to GetGPUDriverInfo()
	FGPUDriverInfo Ret;

	Ret.InternalDriverVersion = TEXT("Unknown");
	Ret.UserDriverVersion = TEXT("Unknown");
	Ret.DriverDate = TEXT("Unknown");

	// for debugging, useful even in shipping to see what went wrong
	FString DebugString;

	uint32 FoundDriverCount = 0;

	int32 Method = CVarDriverDetectionMethod.GetValueOnGameThread();

	if (Method == 5)
	{
		UE_CLOGF(bVerbose, LogWindows, Log, "Gathering driver information using Windows Setup API");
		FGPUDriverInfo Local;
		GetVideoDriverDetailsFromSetup(DeviceDescription, bVerbose, Local);

		if(Local.IsValid() && Local.DeviceDescription == DeviceDescription)
		{
			return Local;
		}

		UE_CLOGF(bVerbose, LogWindows, Log, "Failed to get driver data for device '%ls' using Setup API. Switching to fallback method.", *DeviceDescription);
		Method = 4; // Switch to method 4 as a fallback if method 5 fails
	}
	
	if(Method == 3 || Method == 4)
	{
		UE_CLOGF(bVerbose, LogWindows, Log, "EnumDisplayDevices:");

		for(uint32 i = 0; i < 256; ++i)
		{
			DISPLAY_DEVICE Device;
			
			ZeroMemory(&Device, sizeof(Device));
			Device.cb = sizeof(Device);
			
			// see https://msdn.microsoft.com/en-us/library/windows/desktop/dd162609(v=vs.85).aspx
			if(EnumDisplayDevices(0, i, &Device, EDD_GET_DEVICE_INTERFACE_NAME) == 0)
			{
				// last device or error
				break;
			}

			UE_CLOGF(bVerbose, LogWindows, Log, "   %d. '%ls' (P:%d D:%d), name: '%ls'",
				i,
				Device.DeviceString,
				(Device.StateFlags & DISPLAY_DEVICE_PRIMARY_DEVICE) != 0,
				(Device.StateFlags & DISPLAY_DEVICE_ATTACHED_TO_DESKTOP) != 0,
				Device.DeviceName
				);

			if(Method == 3)
			{
				if (!(Device.StateFlags & DISPLAY_DEVICE_PRIMARY_DEVICE))
				{
					// see http://www.vistaheads.com/forums/microsoft-public-windows-vista-hardware-devices/286017-find-out-active-graphics-device-programmatically-registry-key.html
					DebugString += TEXT("JumpOverNonPrimary ");
					// we want the primary device
					continue;
				}
			}

			FString DriverLocation = Device.DeviceKey;

			if(DriverLocation.Left(18) == TEXT("\\Registry\\Machine\\"))		// not case sensitive
			{
				DriverLocation = FString("\\HKEY_LOCAL_MACHINE\\") + DriverLocation.RightChop(18);
			}
			if(DriverLocation.Left(20) == TEXT("\\HKEY_LOCAL_MACHINE\\"))		// not case sensitive
			{
				FString DriverKey = DriverLocation.RightChop(20);
				
				FGPUDriverInfo Local;
				GetVideoDriverDetails(DriverKey, Local);

				if(!Local.IsValid())
				{
					DebugString += TEXT("GetVideoDriverDetailsInvalid ");
				}

				if((Method == 3) || (Local.DeviceDescription == DeviceDescription))
				{
					if(!FoundDriverCount)
					{
						Ret = Local;
					}
					++FoundDriverCount;
				}
				else
				{
					DebugString += TEXT("PrimaryIsNotTheChoosenAdapter ");
				}
			}
			else
			{
				DebugString += TEXT("PrimaryDriverLocationFailed ");
			}
		}
		
		if(FoundDriverCount != 1)
		{
			// We assume if multiple entries are found they are all the same driver. If that is correct - this is no error.
			DebugString += FString::Printf(TEXT("FoundDriverCount:%d "), FoundDriverCount);
		}

		if(!DebugString.IsEmpty())
		{
			UE_CLOGF(bVerbose, LogWindows, Log, "DebugString: %ls", *DebugString);
		}

		return Ret;
	}

	const bool bIterateAvailableAndChoose = Method == 0;

	if(bIterateAvailableAndChoose)
	{
		for(uint32 i = 0; i < 256; ++i)
		{
			// Iterate all installed display adapters
			const FString DriverNKey = FString::Printf(TEXT("SYSTEM\\CurrentControlSet\\Control\\Class\\{4D36E968-E325-11CE-BFC1-08002BE10318}\\%04d"), i);
		
			FGPUDriverInfo Local;
			GetVideoDriverDetails(DriverNKey, Local);

			if(!Local.IsValid())
			{
				// last device or error
				DebugString += TEXT("GetVideoDriverDetailsInvalid ");
				break;
			}

			if(Local.DeviceDescription == DeviceDescription)
			{
				// found the one we are searching for
				Ret = Local;
				++FoundDriverCount;
				break;
			}
		}
	}

	// FoundDriverCount can be != 1, we take the primary adapter (can be from upgrading a machine to a new OS or old drivers) which also might be wrong
	// see: http://win7settings.blogspot.com/2014/10/how-to-extract-installed-drivers-from.html
	// https://support.microsoft.com/en-us/kb/200435
	// http://www.experts-exchange.com/questions/10198207/Windows-NT-Display-adapter-information.html
	// alternative: from https://support.microsoft.com/en-us/kb/200435
	if(FoundDriverCount != 1)
	{
		// we start again, this time we only look at the primary adapter
		Ret.InternalDriverVersion = TEXT("Unknown");
		Ret.UserDriverVersion = TEXT("Unknown");
		Ret.DriverDate = TEXT("Unknown");

		if(bIterateAvailableAndChoose)
		{
			DebugString += FString::Printf(TEXT("FoundDriverCount:%d FallbackToPrimary "), FoundDriverCount);
		}
	
		FString DriverLocation; // e.g. HKEY_LOCAL_MACHINE\System\CurrentControlSet\Services\<videodriver>\Device0
		// Video0 is the first logical one, not neccesarily the primary, would have to iterate multiple to get the right one (see https://support.microsoft.com/en-us/kb/102992)
		bool bOk = FWindowsPlatformMisc::QueryRegKey(HKEY_LOCAL_MACHINE, TEXT("HARDWARE\\DEVICEMAP\\VIDEO"), TEXT("\\Device\\Video0"), /*out*/ DriverLocation);

		if(bOk)
		{
			if(DriverLocation.Left(18) == TEXT("\\Registry\\Machine\\"))		// not case sensitive
			{
				DriverLocation = FString("\\HKEY_LOCAL_MACHINE\\") + DriverLocation.RightChop(18);
			}
			if(DriverLocation.Left(20) == TEXT("\\HKEY_LOCAL_MACHINE\\"))		// not case sensitive
			{
				FString DriverLocationKey = DriverLocation.RightChop(20);
				
				FGPUDriverInfo Local;
				GetVideoDriverDetails(DriverLocationKey, Local);

				if(!Local.IsValid())
				{
					DebugString += TEXT("GetVideoDriverDetailsInvalid ");
				}

				if(Local.DeviceDescription == DeviceDescription)
				{
					Ret = Local;
					FoundDriverCount = 1;
				}
				else
				{
					DebugString += TEXT("PrimaryIsNotTheChoosenAdapter ");
				}
			}
			else
			{
				DebugString += TEXT("PrimaryDriverLocationFailed ");
			}
		}
		else
		{
			DebugString += TEXT("QueryForPrimaryFailed ");
		}
	}

	if(!DebugString.IsEmpty())
	{
		UE_CLOGF(bVerbose, LogWindows, Log, "DebugString: %ls", *DebugString);
	}

	return Ret;
}

FTDRInfo FWindowsPlatformMisc::GetTDRInfo()
{
	FTDRInfo TDRInfo;
	TDRInfo.Level = ETDRLevel::Recover;
	TDRInfo.Delay = FTDRInfo::DefaultDelay;
	TDRInfo.DdiDelay = FTDRInfo::DefaultDdiDelay;

	const TCHAR* GraphicsDriverKey = TEXT("System\\CurrentControlSet\\Control\\GraphicsDrivers");

	FString TDRLevel;
	if (FWindowsPlatformMisc::QueryRegKey(HKEY_LOCAL_MACHINE, GraphicsDriverKey, TEXT("TdrLevel"), TDRLevel))
	{
		int32 Value = FCString::Atoi(*TDRLevel);
		TDRInfo.Level = (Value >= 0 && Value < int32(ETDRLevel::Invalid)) ? ETDRLevel(Value) : ETDRLevel::Invalid;
	}

	FString TDRDelay;
	if (FWindowsPlatformMisc::QueryRegKey(HKEY_LOCAL_MACHINE, GraphicsDriverKey, TEXT("TdrDelay"), TDRDelay))
	{
		LexFromString(TDRInfo.Delay, *TDRDelay);
	}

	FString TDRDdiDelay;
	if (FWindowsPlatformMisc::QueryRegKey(HKEY_LOCAL_MACHINE, GraphicsDriverKey, TEXT("TdrDdiDelay"), TDRDdiDelay))
	{
		LexFromString(TDRInfo.DdiDelay, *TDRDdiDelay);
	}

	return TDRInfo;
}

#include "Windows/HideWindowsPlatformTypes.h"

void FWindowsPlatformMisc::GetOSVersions( FString& OutOSVersionLabel, FString& OutOSSubVersionLabel )
{
	static struct FOSVersionsInitializer
	{
		FOSVersionsInitializer()
		{
			OSVersionLabel[0] = TEXT('\0');
			OSSubVersionLabel[0] = TEXT('\0');
			GetOSVersionsHelper( OSVersionLabel, UE_ARRAY_COUNT(OSVersionLabel), OSSubVersionLabel, UE_ARRAY_COUNT(OSSubVersionLabel) );
		}

		TCHAR OSVersionLabel[128];
		TCHAR OSSubVersionLabel[128];
	} OSVersionsInitializer;

	OutOSVersionLabel = OSVersionsInitializer.OSVersionLabel;
	OutOSSubVersionLabel = OSVersionsInitializer.OSSubVersionLabel;
}


FString FWindowsPlatformMisc::GetOSVersion()
{
	static struct FOSVersionInitializer
	{
		FOSVersionInitializer()
		{
			CachedOSVersion[0] = TEXT('\0');
			if (!GetOSVersionHelper(CachedOSVersion, UE_ARRAY_COUNT(CachedOSVersion)))
			{
				CachedOSVersion[0] = TEXT('\0');
			}
		}

		TCHAR CachedOSVersion[128];
	} OSVersionInitializer;
	return OSVersionInitializer.CachedOSVersion;
}

bool FWindowsPlatformMisc::GetDiskTotalAndFreeSpace( const FString& InPath, uint64& TotalNumberOfBytes, uint64& NumberOfFreeBytes )
{
	const FString ValidatedPath = FPaths::ConvertRelativePathToFull(InPath).Replace(TEXT("/"), TEXT("\\"));

	bool bSuccess = !!::GetDiskFreeSpaceEx( *ValidatedPath,
											nullptr,
											reinterpret_cast<ULARGE_INTEGER*>(&TotalNumberOfBytes),
											reinterpret_cast<ULARGE_INTEGER*>(&NumberOfFreeBytes));
	return bSuccess;
}

bool FWindowsPlatformMisc::GetPageFaultStats(FPageFaultStats& OutStats, EPageFaultFlags Flags/*=EPageFaultFlags::All*/)
{
	bool bSuccess = false;

	if (EnumHasAnyFlags(Flags, EPageFaultFlags::TotalPageFaults))
	{
		PROCESS_MEMORY_COUNTERS ProcessMemoryCounters;

		FPlatformMemory::Memzero(&ProcessMemoryCounters, sizeof(ProcessMemoryCounters));
		::GetProcessMemoryInfo(::GetCurrentProcess(), &ProcessMemoryCounters, sizeof(ProcessMemoryCounters));

		OutStats.TotalPageFaults = ProcessMemoryCounters.PageFaultCount;

		bSuccess = true;
	}

	return bSuccess;
}

bool FWindowsPlatformMisc::GetBlockingIOStats(FProcessIOStats& OutStats, EInputOutputFlags Flags/*=EInputOutputFlags::All*/)
{
	bool bSuccess = false;
	IO_COUNTERS Counters;

	FPlatformMemory::Memzero(&Counters, sizeof(Counters));

	// Ignore flags as all values are grabbed at once
	if (::GetProcessIoCounters(::GetCurrentProcess(), &Counters) != 0)
	{
		OutStats.BlockingInput = Counters.ReadOperationCount;
		OutStats.BlockingOutput = Counters.WriteOperationCount;
		OutStats.BlockingOther = Counters.OtherOperationCount;
		OutStats.InputBytes = Counters.ReadTransferCount;
		OutStats.OutputBytes = Counters.WriteTransferCount;
		OutStats.OtherBytes = Counters.OtherTransferCount;

		bSuccess = true;
	}

	return bSuccess;
}



uint32 FWindowsPlatformMisc::GetCPUInfo()
{
	return FCPUIDQueriedData::GetCPUInfo();
}

bool FWindowsPlatformMisc::HasTimedPauseCPUFeature()
{
	return FCPUIDQueriedData::HasTimedPauseInstruction();
}

bool FWindowsPlatformMisc::QueryRegKey( const Windows::HKEY InKey, const TCHAR* InSubKey, const TCHAR* InValueName, FString& OutData )
{
	bool bSuccess = false;

	// Redirect key depending on system
	for (int32 RegistryIndex = 0; RegistryIndex < 2 && !bSuccess; ++RegistryIndex)
	{
		HKEY Key = 0;
		const uint32 RegFlags = (RegistryIndex == 0) ? KEY_WOW64_32KEY : KEY_WOW64_64KEY;
		if (RegOpenKeyEx( InKey, InSubKey, 0, KEY_READ | RegFlags, &Key ) == ERROR_SUCCESS)
		{
			::DWORD Size = 0, ValueType = 0;
			// First, we'll call RegQueryValueEx to find out how large of a buffer we need
			if ((RegQueryValueEx( Key, InValueName, NULL, &ValueType, NULL, &Size ) == ERROR_SUCCESS) && Size)
			{
				switch (ValueType)
				{
					case REG_DWORD:
					{
						::DWORD Value;
						if (RegQueryValueEx(Key, InValueName, NULL, NULL, (LPBYTE)&Value, &Size) == ERROR_SUCCESS)
						{
							OutData = FString::Printf(TEXT("%u"), (uint32)Value);
							bSuccess = true;
						}
						break;
					}

					case REG_QWORD:
					{
						int64 Value;
						if (RegQueryValueEx(Key, InValueName, NULL, NULL, (LPBYTE)&Value, &Size) == ERROR_SUCCESS)
						{
							OutData = FString::Printf(TEXT("%" INT64_FMT ), Value);
							bSuccess = true;
						}
						break;
					}

					case REG_SZ:
					case REG_EXPAND_SZ:
					case REG_MULTI_SZ:
					{
						// Allocate a buffer to hold the value and call the function again to get the data
						char* Buffer = new char[Size];
						if (RegQueryValueEx(Key, InValueName, NULL, NULL, (LPBYTE)Buffer, &Size) == ERROR_SUCCESS)
						{
							const uint32 Length = (Size / sizeof(TCHAR)) - 1;
							OutData = FString::ConstructFromPtrSize((TCHAR*)Buffer, Length);
							bSuccess = true;
						}
						delete[] Buffer;
						break;
					}
				}
			}
			RegCloseKey( Key );
		}
	}

	return bSuccess;
}

bool FWindowsPlatformMisc::GetVSComnTools(int32 Version, FString& OutData)
{
	checkf(12 <= Version && Version <= 15, L"Not supported Visual Studio version.");

	FString ValueName = FString::Printf(TEXT("%d.0"), Version);

	FString IDEPath;
	if (!QueryRegKey(HKEY_CURRENT_USER, TEXT("SOFTWARE\\Microsoft\\VisualStudio\\SxS\\VS7"), *ValueName, IDEPath))
	{
		if (!QueryRegKey(HKEY_LOCAL_MACHINE, TEXT("SOFTWARE\\Microsoft\\VisualStudio\\SxS\\VS7"), *ValueName, IDEPath))
		{
			if (!QueryRegKey(HKEY_CURRENT_USER, TEXT("SOFTWARE\\Wow6432Node\\Microsoft\\VisualStudio\\SxS\\VS7"), *ValueName, IDEPath))
			{
				if (!QueryRegKey(HKEY_LOCAL_MACHINE, TEXT("SOFTWARE\\Wow6432Node\\Microsoft\\VisualStudio\\SxS\\VS7"), *ValueName, IDEPath))
				{
					return false;
				}
			}
		}
	}

	OutData = FPaths::ConvertRelativePathToFull(FPaths::Combine(*IDEPath, L"Common7", L"Tools"));
	return true;
}

const TCHAR* FWindowsPlatformMisc::GetDefaultPathSeparator()
{
	return TEXT( "\\" );
}

FText FWindowsPlatformMisc::GetFileManagerName()
{
	return NSLOCTEXT("WindowsPlatform", "FileManagerName", "Explorer");
}

bool FWindowsPlatformMisc::IsDeviceGamingHandheld()
{
#ifndef DEVICEFAMILYDEVICEFORM_GAMING_HANDHELD
	#define DEVICEFAMILYDEVICEFORM_GAMING_HANDHELD    0x0000002E 
#endif

	static const TSet<FString> HandheldAPU_CPUBrandSet =
	{
		TEXT("AMD Custom APU 0405"),	//Steam Deck LCD
		TEXT("AMD Custom APU 0932")		//Steam Deck OLED
	};

	static struct FWindowsHandheldCheck
	{
		bool DeviceHandheld = false;

		FWindowsHandheldCheck()
		{
			// https://learn.microsoft.com/en-us/gaming/gdk/docs/gdk-dev/pc-dev/handheld/handheld-guidance
			DWORD DeviceForm = 0;

			HMODULE Handle = GetModuleHandle(TEXT("ntdll.dll"));
			if (Handle != NULL)
			{
				typedef VOID(NTAPI *RtlGetDeviceFamilyInfoEnum)(_Out_opt_ ULONGLONG*, _Out_opt_ DWORD*, _Out_opt_ DWORD*);
				RtlGetDeviceFamilyInfoEnum RtlGetDeviceFamilyInfo = (RtlGetDeviceFamilyInfoEnum)(void*)GetProcAddress(Handle, "RtlGetDeviceFamilyInfoEnum");
				if (RtlGetDeviceFamilyInfo != NULL)
				{
					RtlGetDeviceFamilyInfo(nullptr, nullptr, &DeviceForm);
				}
			}

			if (DeviceForm == DEVICEFAMILYDEVICEFORM_GAMING_HANDHELD)
			{
				DeviceHandheld = true;
			}

			if (!DeviceHandheld)
			{
				// Some devices are not correctly reporting that they are an Handheld device, check our hard-coded table of Handheld CPU Brand
				if (HandheldAPU_CPUBrandSet.Contains(FWindowsPlatformMisc::GetCPUBrand()))
				{
					DeviceHandheld = true;
				}
			}
		}
	} HandheldCheck;

	return HandheldCheck.DeviceHandheld;
}

static bool IsRunningOnBatteryInternal(SYSTEM_POWER_STATUS Status)
{
	BYTE BatteryPresentFlags = 4  // Critical-the battery capacity is at less than five percent.
		| 2  // Low-the battery capacity is at less than 33 percent.
		| 1  // High-the battery capacity is at more than 66 percent.
		| 8; // Charging.

	BYTE BatteryNotPresentFlags = 128; // No system battery.

	bool BatteryPresent = ((Status.BatteryFlag & BatteryPresentFlags) != 0) && ((Status.BatteryFlag & BatteryNotPresentFlags) == 0);

	// We also need to check for 0, which means "the battery is not being charged and the battery capacity is between low and high".
	return BatteryPresent || Status.BatteryFlag == 0;
}

bool FWindowsPlatformMisc::IsRunningOnBattery()
{
	SYSTEM_POWER_STATUS Status;
	if (GetSystemPowerStatus(&Status) == 0)
	{
		return false;
	}
	return IsRunningOnBatteryInternal(Status);
}

int FWindowsPlatformMisc::GetBatteryLevel()
{
	SYSTEM_POWER_STATUS Status;
	if (GetSystemPowerStatus(&Status) == 0 || !IsRunningOnBatteryInternal(Status))
	{
		return -1; 
	}
	if (Status.BatteryLifePercent == 255)
	{
		return -1; // Unknown battery status
	}
	return Status.BatteryLifePercent;
}

bool FWindowsPlatformMisc::IsUsingBatterySaver()
{
	SYSTEM_POWER_STATUS Status;
	if (GetSystemPowerStatus(&Status) == 0 || !IsRunningOnBatteryInternal(Status))
	{
		return false;
	}
	// A SystemStatusFlag value of 1 indicates battery saver is enabled
	// https://learn.microsoft.com/en-us/windows/win32/api/winbase/ns-winbase-system_power_status
	return Status.SystemStatusFlag == 1;
}

FString FWindowsPlatformMisc::GetOperatingSystemId()
{
	FString Result;
	// more info on this key can be found here: http://stackoverflow.com/questions/99880/generating-a-unique-machine-id
	QueryRegKey(HKEY_LOCAL_MACHINE, TEXT("Software\\Microsoft\\Cryptography"), TEXT("MachineGuid"), Result);
	return Result;
}

FString FWindowsPlatformMisc::GetHostArchitecture()
{
#if PLATFORM_CPU_ARM_FAMILY

	// these build targets can only run on arm64 (assuming there's no emulation)
	return TEXT("arm64");

#else

	// try to find new Windows 10+ API
#pragma warning( push )
#pragma warning( disable: 6387 )	// 'GetModuleHandleW("kernel32")' could be '0'
PRAGMA_DISABLE_CAST_FUNCTION_TYPE_MISMATCH_WARNINGS
	typedef BOOL( WINAPI *LPFN_IsWow64Process2)(HANDLE, USHORT*, USHORT*);
	LPFN_IsWow64Process2 fnIsWow64Process2 = (LPFN_IsWow64Process2)GetProcAddress( GetModuleHandle(TEXT("kernel32")), "IsWow64Process2" );
PRAGMA_ENABLE_CAST_FUNCTION_TYPE_MISMATCH_WARNINGS
#pragma warning( pop )

	USHORT ProcessMachine, NativeMachine;
	if (fnIsWow64Process2 != nullptr && fnIsWow64Process2(::GetCurrentProcess(), &ProcessMachine, &NativeMachine) )
	{
		if (NativeMachine == IMAGE_FILE_MACHINE_ARM64)
		{
			return TEXT("arm64");
		}
	}

	// assume normal x64 by default
	return "x64";

#endif
}


EConvertibleLaptopMode FWindowsPlatformMisc::GetConvertibleLaptopMode()
{
	if (!VerifyWindowsVersion(6, 2))
	{
		return EConvertibleLaptopMode::NotSupported;
	}

	if (::GetSystemMetrics(SM_CONVERTIBLESLATEMODE) == 0)
	{
		return EConvertibleLaptopMode::Tablet;
	}
	
	return EConvertibleLaptopMode::Laptop;
}

IPlatformChunkInstall* FWindowsPlatformMisc::GetPlatformChunkInstall()
{
	static IPlatformChunkInstall* ChunkInstall = nullptr;
	static bool bIniChecked = false;
	if (!ChunkInstall || !bIniChecked)
	{
		IPlatformChunkInstallModule* PlatformChunkInstallModule = nullptr;
		if (!GEngineIni.IsEmpty())
		{
			FString InstallModule;
			GConfig->GetString(TEXT("StreamingInstall"), TEXT("DefaultProviderName"), InstallModule, GEngineIni);

			if (!InstallModule.IsEmpty())
			{			
				PlatformChunkInstallModule = FModuleManager::LoadModulePtr<IPlatformChunkInstallModule>(*InstallModule);
				if (PlatformChunkInstallModule != nullptr)
				{
					// Attempt to grab the platform installer
					ChunkInstall = PlatformChunkInstallModule->GetPlatformChunkInstall();
				}
			}
			bIniChecked = true;
		}

		if (PlatformChunkInstallModule == nullptr)
		{
			// Placeholder instance
			ChunkInstall = FGenericPlatformMisc::GetPlatformChunkInstall();
		}
	}

	return ChunkInstall;
}

void FWindowsPlatformMisc::PumpMessagesOutsideMainLoop()
{
	TGuardValue<bool> PumpMessageGuard(GPumpingMessagesOutsideOfMainLoop, true);
	// Process pending windows messages, which is necessary to the rendering thread in some cases where D3D
	// sends window messages (from IDXGISwapChain::Present) to the main thread owned viewport window.
	MSG Msg;
	PeekMessage(&Msg, NULL, 0, 0, PM_NOREMOVE | PM_QS_SENDMESSAGE);
	return;
}

uint64 FWindowsPlatformMisc::GetFileVersion(const FString &FileName)
{
	::DWORD VersionInfoSize = GetFileVersionInfoSize(*FileName, NULL);
	if (VersionInfoSize != 0)
	{
		TArray<uint8> VersionInfo;
		VersionInfo.AddUninitialized(VersionInfoSize);
		if (GetFileVersionInfo(*FileName, NULL, VersionInfoSize, VersionInfo.GetData()))
		{
			VS_FIXEDFILEINFO *FileInfo;
			::UINT FileInfoLen;
			if (VerQueryValue(VersionInfo.GetData(), TEXT("\\"), (LPVOID*)&FileInfo, &FileInfoLen))
			{
				return (uint64(FileInfo->dwFileVersionMS) << 32) | uint64(FileInfo->dwFileVersionLS);
			}
		}
	}
	return 0;
}

int32 FWindowsPlatformMisc::GetMaxRefreshRate()
{
	int32 Result = FGenericPlatformMisc::GetMaxRefreshRate();

#if !UE_SERVER
	DEVMODE DeviceMode;
	FMemory::Memzero(DeviceMode);
	DeviceMode.dmSize = sizeof(DEVMODE);

	if (EnumDisplaySettings(NULL, ENUM_CURRENT_SETTINGS, &DeviceMode) != 0)
	{
		// dmDisplayFrequency isn't always useful, the Windows docs say it can
		// return 0 or 1 to indicate 'default refresh rate', so always assume we
		// can do at least the generic platform default of 60 Hz
		Result = FMath::Max(Result, (int32)DeviceMode.dmDisplayFrequency);
	}
#endif

	return Result;
}

void FWindowsPlatformMisc::UpdateDriveFreeSpace()
{
	WaitOnCollectStorageInformationAsync();
	for (auto& StorageDevice : StorageDevices)
	{
		ULARGE_INTEGER TotalNumberOfBytes, TotalNumberOfFreeBytes;
		WCHAR DriveName[4] = { StorageDevice.Stats.DriveName, L':', L'\\', 0 };
		if (GetDiskFreeSpaceExW(DriveName, NULL, &TotalNumberOfBytes, &TotalNumberOfFreeBytes))
		{
			StorageDevice.Stats.FreeBytes = TotalNumberOfFreeBytes.QuadPart;
			StorageDevice.Stats.UsedBytes = TotalNumberOfBytes.QuadPart - TotalNumberOfFreeBytes.QuadPart;
		}
	}
}

const FPlatformDriveStats* FWindowsPlatformMisc::GetDriveStats(WIDECHAR DriveLetter)
{
	WaitOnCollectStorageInformationAsync();
	for (auto& StorageDevice : StorageDevices)
	{
		if (StorageDevice.Stats.DriveName == DriveLetter)
		{
			return &StorageDevice.Stats;
		}
	}
	return nullptr;
}

void FWindowsPlatformMisc::ApplyWindowsCompatConfigRules(void)
{
#if WITH_APPLICATION_CORE && WITH_WINDOWS_CONFIGRULES

	bool bSkipAllowList = false;
#if !UE_BUILD_SHIPPING
	bSkipAllowList = FParse::Param(FCommandLine::Get(), TEXT("SkipConfigRulesAllowList"));
#endif

	TArray<FString> WindowsCompatCVars;
	if (bSkipAllowList || GConfig->GetArray(TEXT("WindowsCompatCVars"), TEXT("CVars"), WindowsCompatCVars, GEngineIni))
	{
		TSet<FString> AllowedCompatCVars(WindowsCompatCVars);
		const TMap<FString, FString>& ConfigRuleVars = FPlatformMisc::GetConfigRuleVars();
		for (const TTuple<FString, FString>& Pair : ConfigRuleVars)
		{
			const FString& Key = Pair.Key;
			const FString& Value = Pair.Value;
			static const TCHAR WindowsCompat[] = TEXT("WindowsCompat.");
			if (Key.StartsWith(WindowsCompat))
			{
				FString CVarName = Key.Mid(UE_ARRAY_COUNT(WindowsCompat) - 1);
				if (bSkipAllowList || AllowedCompatCVars.Contains(CVarName))
				{
					auto* CVar = IConsoleManager::Get().FindConsoleVariable(*CVarName);
					if (CVar)
					{
						// set with HF priority
						// configrules are therefore higher priority than deviceprofiles. (e.g. -dpcvars)
						CVar->Set(*Value, ECVF_SetByHotfix);
						UE_LOGF(LogConfigRules, Log, "Compat Setting %ls = %ls", *CVarName, *Value);
					}
				}
			}
		}
	}
#endif // WITH_APPLICATION_CORE && WITH_WINDOWS_CONFIGRULES
}

const class TMap<FString, FString>& FWindowsPlatformMisc::GetConfigRuleVars()
{
#if WITH_APPLICATION_CORE && WITH_WINDOWS_CONFIGRULES
	return FWindowsConfigRules::GetConfigRulesMap();
#else 
	return FGenericPlatformMisc::GetConfigRuleVars();
#endif // WITH_APPLICATION_CORE && WITH_WINDOWS_CONFIGRULES
}

static INetworkListManager* GNetworkListManager = nullptr;
static bool GNetworkMonitoringInitializedCOM = false;

/** Cached network connection type, updated by NLM events and initial query.
  * Under the current STA apartment model, both the ConnectivityChanged callback and all reads are serialized on the game thread via window message marshaling. */
static ENetworkConnectionType GCachedNetworkConnectionType = ENetworkConnectionType::Unknown;

/**
 * Queries NLM network connections to determine if the active connection is WiFi or wired Ethernet
 */
static ENetworkConnectionType DetermineConnectionTypeFromNLM(INetworkListManager* NetworkListManager)
{
	// Get an enumerator over all NLM network connections
	IEnumNetworkConnections* EnumConnections = nullptr;
	HRESULT Result = NetworkListManager->GetNetworkConnections(&EnumConnections);
	if (FAILED(Result) || EnumConnections == nullptr)
	{
		return ENetworkConnectionType::Unknown;
	}

	bool bHasWiFi = false;
	bool bHasEthernet = false;

	// Iterate each connection and check only those that are currently connected
	INetworkConnection* Connection = nullptr;
	while (EnumConnections->Next(1, &Connection, nullptr) == S_OK)
	{
		VARIANT_BOOL IsConnected = VARIANT_FALSE;
		if (SUCCEEDED(Connection->get_IsConnected(&IsConnected)) && IsConnected != VARIANT_FALSE)
		{
			// Get the adapter GUID that backs this NLM connection
			GUID AdapterId;
			if (SUCCEEDED(Connection->GetAdapterId(&AdapterId)))
			{
				// Convert adapter GUID -> LUID -> interface index.
				// We need the index because MIB_IFROW (used by GetIfEntry) is keyed by dwIndex
				
				NET_LUID Luid;
				NET_IFINDEX IfIndex;

				if (ConvertInterfaceGuidToLuid(&AdapterId, &Luid) == NO_ERROR &&
					ConvertInterfaceLuidToIndex(&Luid, &IfIndex) == NO_ERROR)
				{
					// Query the interface row to read the physical adapter type
					
					MIB_IFROW IfRow = {};
					IfRow.dwIndex = IfIndex;

					if (GetIfEntry(&IfRow) == NO_ERROR)
					{
						if (IfRow.dwType == IF_TYPE_IEEE80211)
						{
							bHasWiFi = true;
						}
						else if (IfRow.dwType == IF_TYPE_ETHERNET_CSMACD)
						{
							bHasEthernet = true;
						}
					}
				}
			}
		}

		Connection->Release();
	}

	EnumConnections->Release();

	// When both WiFi and Ethernet are active (e.g. docked laptop), prefer Ethernet, as the wired connection is typically the primary route
	
	if (bHasEthernet)
	{
		return ENetworkConnectionType::Ethernet;
	}

	if (bHasWiFi)
	{
		return ENetworkConnectionType::WiFi;
	}

	// Fallback: no recognized adapter type was found (e.g. VPN or cellular)
	return ENetworkConnectionType::Unknown;
}

/**
 * Queries NLM for the current network connection type. Expensive as it involves COM calls and interface enumeration.
 * Should only be called during init and from NLM event callbacks, not per-frame.
 * The result is stored in GCachedNetworkConnectionType.
 */
static ENetworkConnectionType QueryNetworkConnectionTypeFromNLM()
{
	if (GNetworkListManager == nullptr)
	{
		return ENetworkConnectionType::Unknown;
	}

	NLM_CONNECTIVITY Connectivity;
	HRESULT GetConnectivityResult = GNetworkListManager->GetConnectivity(&Connectivity);
	if (SUCCEEDED(GetConnectivityResult))
	{
		if (Connectivity & (NLM_CONNECTIVITY_IPV4_INTERNET | NLM_CONNECTIVITY_IPV6_INTERNET |
			NLM_CONNECTIVITY_IPV4_LOCALNETWORK | NLM_CONNECTIVITY_IPV6_LOCALNETWORK))
		{
			return DetermineConnectionTypeFromNLM(GNetworkListManager);
		}
		else
		{
			return ENetworkConnectionType::None;
		}
	}

	return ENetworkConnectionType::Unknown;
}

/**
 * COM event listener that receives network connectivity change notifications from NLM.
 * Registered in StartNetworkConnectionMonitoring, unregistered in StopNetworkConnectionMonitoring.
 */
class FNLMEventListener : public INetworkListManagerEvents
{
public:
	FNLMEventListener() : RefCount(1), Cookie(0), ConnectionPoint(nullptr) {}
	virtual ~FNLMEventListener() = default;

	// IUnknown interface

	HRESULT STDMETHODCALLTYPE QueryInterface(REFIID riid, void** ppvObject) override
	{
		if (riid == IID_IUnknown || riid == IID_INetworkListManagerEvents)
		{
			*ppvObject = this;
			AddRef();
			return S_OK;
		}
		*ppvObject = nullptr;
		return E_NOINTERFACE;
	}

	ULONG STDMETHODCALLTYPE AddRef() override
	{
		// We use the PlatformAtomics methods to avoid having to add this to AllowWindowsPlatformTypes
		return FPlatformAtomics::InterlockedIncrement(&RefCount);
	}

	ULONG STDMETHODCALLTYPE Release() override
	{
		// We use the PlatformAtomics methods to avoid having to add this to AllowWindowsPlatformTypes
		int32 Ref = FPlatformAtomics::InterlockedDecrement(&RefCount);
		if (Ref == 0)
		{
			delete this;
		}
		return Ref;
	}

	// ~ IUnknown interface

	// INetworkListManagerEvents

	HRESULT STDMETHODCALLTYPE ConnectivityChanged(NLM_CONNECTIVITY /*NewConnectivity*/) override
	{
		// We ignore the raw NLM_CONNECTIVITY bitmask and re-query NLM to also resolve adapter types.
		const ENetworkConnectionType ConnectionType = QueryNetworkConnectionTypeFromNLM();
		const ENetworkConnectionType PreviousType = GCachedNetworkConnectionType;
		GCachedNetworkConnectionType = ConnectionType;

		// NLM can fire ConnectivityChanged for events that don't affect the resolved type (e.g. a VPN connecting while Ethernet stays primary). Skip redundant broadcasts.
		if (ConnectionType == PreviousType)
		{
			return S_OK;
		}

		UE_LOGF(LogInit, Verbose, "Network connectivity changed (ConnectionType=%d)", static_cast<int32>(ConnectionType));

		if (FTaskGraphInterface::IsRunning())
		{
			FFunctionGraphTask::CreateAndDispatchWhenReady([ConnectionType]()
			{
				FCoreDelegates::OnNetworkConnectionChanged.Broadcast(ConnectionType);
				FPlatformMisc::SetNetworkConnectionStatus(FPlatformMisc::NetworkConnectionTypeToStatus(ConnectionType));
			}, TStatId(), nullptr, ENamedThreads::GameThread);
		}
		else
		{
			// Task graph not running yet (early startup). Set the status synchronously.
			FPlatformMisc::SetNetworkConnectionStatus(FPlatformMisc::NetworkConnectionTypeToStatus(ConnectionType));
		}
		return S_OK;
	}

	// ~ INetworkListManagerEvents

	/** Subscribe to NLM connectivity events. Returns true on success. */
	bool Advise(INetworkListManager* NetworkListManager)
	{
		IConnectionPointContainer* Container = nullptr;
		HRESULT Result = NetworkListManager->QueryInterface(IID_IConnectionPointContainer, (void**)&Container);
		if (FAILED(Result) || Container == nullptr)
		{
			return false;
		}

		Result = Container->FindConnectionPoint(IID_INetworkListManagerEvents, &ConnectionPoint);
		Container->Release();
		if (FAILED(Result) || ConnectionPoint == nullptr)
		{
			return false;
		}

		Result = ConnectionPoint->Advise(this, &Cookie);
		if (FAILED(Result))
		{
			ConnectionPoint->Release();
			ConnectionPoint = nullptr;
			return false;
		}

		return true;
	}

	/** Unsubscribe from NLM connectivity events. */
	void Unadvise()
	{
		if (ConnectionPoint != nullptr)
		{
			if (Cookie != 0)
			{
				ConnectionPoint->Unadvise(Cookie);
				Cookie = 0;
			}
			ConnectionPoint->Release();
			ConnectionPoint = nullptr;
		}
	}

private:
	// Used by the IUnknown interface, type required by InterlockedIncrement/Decrement methods
	volatile int32 RefCount;

	// Token returned by Advise which identifies our registration
	DWORD Cookie;

	// Intermediary between the event source (INetworkListManager) and this listener
	IConnectionPoint* ConnectionPoint;
};

static FNLMEventListener* GNLMEventListener = nullptr;

ENetworkConnectionType FWindowsPlatformMisc::GetNetworkConnectionType()
{
	return GCachedNetworkConnectionType;
}

void FWindowsPlatformMisc::StartNetworkConnectionMonitoring()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FWindowsPlatformMisc::StartNetworkConnectionMonitoring);

	check(GNetworkListManager == nullptr && GNLMEventListener == nullptr);

	// COM must be initialized on this thread before we can create COM objects.
	// This defaults to a single-threaded apartment (COINIT_APARTMENTTHREADED), which means NLM callbacks are marshaled to this thread via window messages.
	// Callbacks will only fire when the game thread pumps messages (FWindowsPlatformApplicationMisc::PumpMessages), so there may be latency under heavy game-thread load.
	// This also guarantees that callbacks and teardown are serialized on the same thread, avoiding races.
	if (!FWindowsPlatformMisc::CoInitialize())
	{
		UE_LOGF(LogInit, Log, "Failed to initialize COM for network monitoring");
		return;
	}
	
	GNetworkMonitoringInitializedCOM = true;

	HRESULT CoCreateInstanceResult = CoCreateInstance(CLSID_NetworkListManager, nullptr, CLSCTX_ALL, IID_INetworkListManager, (void**)&GNetworkListManager);
	if (FAILED(CoCreateInstanceResult) || GNetworkListManager == nullptr)
	{
		UE_LOGF(LogInit, Log, "Failed to create INetworkListManager for network monitoring (HRESULT=0x%08X)", CoCreateInstanceResult);
		GNetworkListManager = nullptr;
		FWindowsPlatformMisc::CoUninitialize();
		GNetworkMonitoringInitializedCOM = false;
		return;
	}

	GNLMEventListener = new FNLMEventListener();
	if (!GNLMEventListener->Advise(GNetworkListManager))
	{
		UE_LOGF(LogInit, Log, "Failed to subscribe to INetworkListManagerEvents");
		GNLMEventListener->Release();
		GNLMEventListener = nullptr;
		GNetworkListManager->Release();
		GNetworkListManager = nullptr;
		FWindowsPlatformMisc::CoUninitialize();
		GNetworkMonitoringInitializedCOM = false;
		return;
	}

	// Set the initial network connection status.
	const ENetworkConnectionType InitialConnectionType = QueryNetworkConnectionTypeFromNLM();
	GCachedNetworkConnectionType = InitialConnectionType;
	SetNetworkConnectionStatus(NetworkConnectionTypeToStatus(InitialConnectionType));

	// Register cleanup on OnEnginePreExit - this fires before OnPreExit and OnExit.
	// We must clean it up earlier than PlatformTearDown because its callbacks can crash at that point
	FCoreDelegates::OnEnginePreExit.AddStatic(StopNetworkConnectionMonitoring);
}

void FWindowsPlatformMisc::StopNetworkConnectionMonitoring()
{
	// Safe to access GNetworkListManager without a lock here: under the single-threaded COM apartment (COINIT_APARTMENTTHREADED), both the ConnectivityChanged callback and this teardown run on the same thread (the game thread), so they are serialized.
	// If the apartment model is ever changed to multithreaded (COINIT_MULTITHREADED), a mutex or ref-counting guard would be needed.
	if (GNLMEventListener != nullptr)
	{
		GNLMEventListener->Unadvise();
		GNLMEventListener->Release();
		GNLMEventListener = nullptr;
	}

	if (GNetworkListManager != nullptr)
	{
		GNetworkListManager->Release();
		GNetworkListManager = nullptr;
	}

	if (GNetworkMonitoringInitializedCOM)
	{
		FWindowsPlatformMisc::CoUninitialize();
		GNetworkMonitoringInitializedCOM = false;
	}
}

#undef UE_CPUID
