// Copyright Epic Games, Inc. All Rights Reserved.

#include "Windows/WindowsPlatformMutex.h"

#include "Containers/UnrealString.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/AssertionMacros.h"
#include "Windows/AllowWindowsPlatformTypes.h"

namespace UE
{

FWindowsSystemWideMutex::FWindowsSystemWideMutex(const FString& InName, FTimespan InTimeout)
{
	// Check for valid name length not exceeding the Window mutex name length limit
	check(InName.Len() > 0)
	check(InName.Len() < MAX_PATH)
	// Check for valid positive timeouts
	check(InTimeout >= FTimespan::Zero())
	check(InTimeout.GetTotalMilliseconds() < (double)0x7FFFFFFE)	// limit timespan to a number of millisec that will fit in a signed int32

	// Disallow backslashes as they aren't allowed in Windows mutex names
	FString SanitizedName(InName);
	FGenericPlatformMisc::SanitizeSystemWideMutexName(SanitizedName);

	TCHAR MutexName[MAX_PATH] = {};
	FCString::Strcpy(MutexName, *SanitizedName);

	// Attempt to create and take ownership of a named mutex
	Mutex = CreateMutex(NULL, true, MutexName);

	if (Mutex != NULL && GetLastError() == ERROR_ALREADY_EXISTS)
	{
		// CreateMutex returned a valid handle but we didn't get ownership because another process/thread has already created it
		bool bMutexOwned = false;

		if (InTimeout != FTimespan::Zero())
		{
			// We have a handle already so try waiting for it to be released by the current owner
			DWORD WaitResult = WaitForSingleObject(Mutex, FMath::TruncToInt((float)InTimeout.GetTotalMilliseconds()));

			// WAIT_OBJECT_0 = we got ownership when the previous owner released it
			// WAIT_ABANDONED = we got ownership when the previous owner exited WITHOUT releasing the mutex gracefully (we own it now but the state of any shared resource could be corrupted!)
			if (WaitResult == WAIT_ABANDONED || WaitResult == WAIT_OBJECT_0)
			{
				bMutexOwned = true;
			}
		}

		if (!bMutexOwned)
		{
			// We failed to gain ownership by waiting so close the handle to avoid leaking it.
			CloseHandle(Mutex);
			Mutex = NULL;
		}
	}
}

FWindowsSystemWideMutex::~FWindowsSystemWideMutex()
{
	Release();
}

bool FWindowsSystemWideMutex::IsValid() const
{
	return Mutex != NULL;
}

void FWindowsSystemWideMutex::Release()
{
	if (IsValid())
	{
		// Release ownership
		ReleaseMutex(Mutex);
		// Also release the handle so it isn't leaked
		CloseHandle(Mutex);
		Mutex = NULL;
	}
}

} // UE

#include "Windows/HideWindowsPlatformTypes.h"
