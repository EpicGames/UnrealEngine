// Copyright Epic Games, Inc. All Rights Reserved.

#include "Windows/WindowsPlatformNamedPipe.h"
#include "Templates/IsPointer.h"
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformMisc.h"
#include "HAL/PlatformProcess.h"
#include "Misc/AssertionMacros.h"
#include "Logging/LogMacros.h"
#include "HAL/UnrealMemory.h"
#include "Misc/CString.h"
#include "Containers/UnrealString.h"
#include "CoreGlobals.h"
#include "Windows/WindowsHWrapper.h"

static TAutoConsoleVariable<bool> CVarWindowsLogNamedPipeEvents(
	TEXT("Windows.LogNamedPipeEvents"),
	false,
	TEXT("Whether to log named pipe activity"),
	ECVF_Default);

static void VerifyWinResult(BOOL bResult, const TCHAR* InMessage)
{
	if (!bResult)
	{
		TCHAR ErrorBuffer[ MAX_SPRINTF ];
		uint32 Error = GetLastError();
		const TCHAR* Buffer = FWindowsPlatformMisc::GetSystemErrorMessage(ErrorBuffer, MAX_SPRINTF, Error);
		FString Message = FString::Printf(TEXT("FAILED (%s) with GetLastError() %d: %s!\n"), InMessage, Error, ErrorBuffer);
		FPlatformMisc::LowLevelOutputDebugString(*Message);
		UE_LOGF(LogWindows, Fatal, "%ls", *Message);
	}
	verify(bResult != 0);
}

#if PLATFORM_SUPPORTS_NAMED_PIPES
FWindowsPlatformNamedPipe::FWindowsPlatformNamedPipe() :
	Pipe(INVALID_HANDLE_VALUE),
	bUseOverlapped(false),
	bIsServer(false),
	State(FPlatformNamedPipe::State_Uninitialized)
{
	FMemory::Memzero(Overlapped);
	LogInternals("(Constructor)");
}

FWindowsPlatformNamedPipe::~FWindowsPlatformNamedPipe()
{
}

bool FWindowsPlatformNamedPipe::Create(const FString& PipeName, bool bAsServer, bool bAsync)
{
	check(State == State_Uninitialized);
	FString& Name = *NamePtr;

	Name = PipeName;
	bIsServer = bAsServer;
	if (bAsServer)
	{
		uint32 OpenModeFlags = PIPE_ACCESS_DUPLEX | FILE_FLAG_FIRST_PIPE_INSTANCE;
		if (bAsync)
		{
			OpenModeFlags |= FILE_FLAG_OVERLAPPED;
		}
		//FPlatformMisc::LowLevelOutputDebugStringf(*FString::Printf(TEXT("*** %s: Create Pipe\n"), *Name));
		Pipe = CreateNamedPipe(*Name, OpenModeFlags, PIPE_TYPE_BYTE | PIPE_WAIT, 1, 0, 0, 0, NULL);				
	}
	else
	{
		uint32 Flags = FILE_ATTRIBUTE_NORMAL;
		if (bAsync)
		{
			Flags |= FILE_FLAG_OVERLAPPED;
		}

		Pipe = CreateFile(*Name,
			GENERIC_READ | GENERIC_WRITE,
			FILE_SHARE_READ | FILE_SHARE_WRITE,
			NULL,
			OPEN_EXISTING,
			Flags,
			NULL
			);
	}

	const bool bSuccess = Pipe != INVALID_HANDLE_VALUE;
	if (bSuccess)
	{
		FMemory::Memzero(Overlapped);

		State = bAsServer ? State_Created : State_ReadyForRW;

		bUseOverlapped = bAsync;
		if (bAsync)
		{
			Overlapped.hEvent = CreateEventW(NULL, true, true, NULL);
			VerifyWinResult(Overlapped.hEvent != 0, TEXT("CreateEventW"));
		}

		LogInternals("Create=Success");
	}
	else
	{
		LogInternals("Create=Fail");
	}

	return bSuccess;
}

bool FWindowsPlatformNamedPipe::Destroy()
{
	FString& Name = *NamePtr;

	bool bFlushBuffers = false;
	bool bDisconnect = true;
	switch (State)
	{
		case State_Created:
			bFlushBuffers = false;
			bDisconnect = false;
			break;

		case State_ReadyForRW:
			// Nothing to do here, carry on
			break;

		case State_WaitingForRW:
			// TODO: Cancel IO and wait
			bFlushBuffers = true;
			break;

		case State_Uninitialized:
			// No need to destroy since it wasn't created
			return true;

		case State_Connecting:
			// TODO: Cancel IO
			break;

		case State_ErrorPipeClosedUnexpectedly:
			bFlushBuffers = false;
			bDisconnect = false;
			break;

		default:
			// Invalid state!
			check(0);
			return false;
	}

	//FPlatformMisc::LowLevelOutputDebugStringf(*FString::Printf(TEXT("*** %s: Closing connection\n"), *Name));
	if (bFlushBuffers)
	{
		VerifyWinResult(FlushFileBuffers(Pipe), TEXT("Flushing File Buffers"));
	}

	if (bDisconnect && bIsServer)
	{
		VerifyWinResult(DisconnectNamedPipe(Pipe), TEXT("Disconnecting Named Pipe"));
	}

	if (bUseOverlapped)
	{
		switch (WaitForSingleObject(Overlapped.hEvent, 0))
		{
		case WAIT_TIMEOUT:
			if (Windows::CancelIoEx(Pipe, &Overlapped))
			{
				VerifyWinResult(WaitForSingleObject(Overlapped.hEvent, INFINITE) == WAIT_OBJECT_0, TEXT("Wait for Overlapped cancel"));
			}
			else
			{
				VerifyWinResult(GetLastError() == ERROR_NOT_FOUND, TEXT("CancelIoEx failed for an unexpected reason"));
			}
			[[fallthrough]];
		case WAIT_OBJECT_0:
			VerifyWinResult(CloseHandle(Overlapped.hEvent), TEXT("Close Overlapped.hEvent"));
			break;
		case WAIT_FAILED:
		default:
			VerifyWinResult(false, TEXT("Failed to wait for Overlapped.hEvent"));
			break;
		}

		bUseOverlapped = false;
	}

	VerifyWinResult(CloseHandle(Pipe), TEXT("Closing Handle"));
	Pipe = INVALID_HANDLE_VALUE;
	Name = TEXT("");

	State = State_Uninitialized;

	LogInternals("Destroy");
	return true;
}

bool FWindowsPlatformNamedPipe::OpenConnection()
{
	check(bIsServer);
	check(State == State_Created);
	//FPlatformMisc::LowLevelOutputDebugStringf(*FString::Printf(TEXT("*** %s: Opening connection\n"), *Name));

	uint32 Result = Windows::ConnectNamedPipe(Pipe, bUseOverlapped ? &Overlapped : nullptr);
	if (!bUseOverlapped)
	{
		if (Result == 0)
		{
			uint32 LastError = GetLastError();
			if (LastError != ERROR_PIPE_CONNECTED)
			{
				LogInternals("OpenConnection=SyncFail");
				VerifyWinResult(Result, TEXT("During OpenConnection()"));
				return false;
			}
		}

		LogInternals("OpenConnection=SyncSuccess");
		State = State_ReadyForRW;
		return true;
	}

	// Overlapped
	if (!Result)
	{
		// The client might have connected just before this function was called, which is correct
		uint32 LastError = GetLastError();
		switch (LastError)
		{
			case ERROR_IO_PENDING:
				State = State_Connecting;
				break;

			case ERROR_PIPE_CONNECTED:
				State = State_ReadyForRW;
				break;

			case ERROR_BROKEN_PIPE:
			case ERROR_NO_DATA:
			case ERROR_PIPE_NOT_CONNECTED:
			case ERROR_BAD_PIPE:
				State = State_ErrorPipeClosedUnexpectedly;
				break;

			default:
				VerifyWinResult(false, TEXT("During OpenConnection()"));
				return false;
		}
	}

	LogInternals("OpenConnection=Pending");
	return true;
}

bool FWindowsPlatformNamedPipe::BlockForAsyncIO()
{
	bool bTryAgain = false;
	do
	{
		bTryAgain = false;
		// Yield CPU time while waiting
		FPlatformProcess::Sleep(.01f);
		if (!UpdateAsyncStatus())
		{
			return false;
		}

		switch (State)
		{
			case State_Uninitialized:
				UE_LOGF(LogWindows, Fatal, "Need to Create() first!!\n");
				check(0);
				break;

			case State_Created:
			case State_ReadyForRW:
				// Done
				return true;

			case State_Connecting:
			case State_WaitingForRW:
				check(bUseOverlapped);
				bTryAgain = true;
				break;

			case State_ErrorPipeClosedUnexpectedly:
				// Should never get here!
			default:
				// Invalid state!
				check(0);
				break;
		}
	}
	while (bTryAgain); //-V654

	return true;
}

bool FWindowsPlatformNamedPipe::UpdateAsyncStatusAfterRW()
{
	check(bUseOverlapped);
	uint32 LastError = GetLastError();
	switch (LastError)
	{
		case ERROR_IO_PENDING:
			// Yield CPU time while waiting
			FPlatformProcess::Sleep(.01f);
			return true;

		case ERROR_NO_DATA:
			// The pipe got closed
			State = State_ErrorPipeClosedUnexpectedly;
			LogInternals("UpdateAsyncStatusAfterRW=ERROR_NO_DATA");
			break;

		case ERROR_BROKEN_PIPE:
			State = State_ErrorPipeClosedUnexpectedly;
			LogInternals("UpdateAsyncStatusAfterRW=ERROR_BROKEN_PIPE");
			break;

		default:
			VerifyWinResult(false, TEXT("During UpdateAsyncStatusAfterRW()"));
			break;
	}

	return false;
}

bool FWindowsPlatformNamedPipe::IsReadyForRW() const
{
	return (State == State_ReadyForRW);
}

bool FWindowsPlatformNamedPipe::UpdateAsyncStatus()
{
	switch (State)
	{
		case State_Connecting:
		case State_WaitingForRW:
			{
				check(bUseOverlapped);

				// Query IO state
				Windows::DWORD Unused = 0;
				Windows::BOOL Result = Windows::GetOverlappedResult(Pipe, &Overlapped, &Unused, false);
				if (Result)
				{
					// Finished
					State = State_ReadyForRW;
					LogInternals("UpdateAsyncStatus=ReadyForRW");
				}
				else
				{
					uint32 LastError = GetLastError();
					switch (LastError)
					{
						case ERROR_IO_INCOMPLETE:
							break;

						case ERROR_BROKEN_PIPE:
							State = State_ErrorPipeClosedUnexpectedly;
							LogInternals("UpdateAsyncStatus=ErrorPipeClosedUnexpectedly");
							return false;

						default:
							VerifyWinResult(false, TEXT("During UpdateAsyncStatus()"));
							break;
					}
				}
			}
			break;

		case State_ReadyForRW:
		case State_Created:
			// Nothing to do here, carry on
			break;

		case State_Uninitialized:
		case State_ErrorPipeClosedUnexpectedly:
			return false;

		default:
			// Invalid state!
			check(0);
			break;
	}

	return true;
}

bool FWindowsPlatformNamedPipe::WriteBytes(int32 NumBytes, const void* Data)
{
	check(NumBytes > 0);
	check(Data);

	switch (State)
	{
		case State_Created:
			UE_LOGF(LogWindows, Fatal, "Need to OpenConnection() first!!\n");
			check(0);
			break;

		case State_ReadyForRW:
			// Nothing to do here, carry on
			break;

		case State_Uninitialized:
			UE_LOGF(LogWindows, Fatal, "Need to Create() first!!\n");
			check(0);
			break;

		case State_WaitingForRW:
		case State_Connecting:
			// Need to wait outside the client!
			check(0);
			break;

		case State_ErrorPipeClosedUnexpectedly:
			return false;

		default:
			// Invalid state!
			check(0);
			break;
	}

	//FPlatformMisc::LowLevelOutputDebugStringf(*FString::Printf(TEXT("*** %s: Writing %d bytes\n"), *Name, NumBytes));

	uint64 WrittenBytes = 0;
	Windows::LPDWORD WrittenBytesPtr = bUseOverlapped ? nullptr : (Windows::LPDWORD)&WrittenBytes;
	BOOL Result = Windows::WriteFile(Pipe, Data, NumBytes, WrittenBytesPtr, bUseOverlapped ? &Overlapped : nullptr);
	if (!bUseOverlapped)
	{
		//FPlatformMisc::LowLevelOutputDebugStringf(*FString::Printf(TEXT("\t\tActually wrote %d bytes\n"), WrittenBytes));
		if (!Result)
		{
			uint32 LastError = GetLastError();
			switch (LastError)
			{
				case ERROR_BROKEN_PIPE:
				case ERROR_NO_DATA:
					State = State_ErrorPipeClosedUnexpectedly;
					LogInternals("WriteBytes=ErrorPipeClosedUnexpectedly");
					break;

				default:
					VerifyWinResult(0, TEXT("During WriteBytes()"));
					break;
			}
		}

		return (Result && WrittenBytes == NumBytes);
	}

	if (Result)
	{
		// Operation finished
		State = State_ReadyForRW;
		LogInternals("WriteBytes=State_ReadyForRW");
		return true;
	}

	if (!UpdateAsyncStatusAfterRW())
	{
		return false;
	}

	State = State_WaitingForRW;
	LogInternals("WriteBytes=State_WaitingForRW");
	return UpdateAsyncStatus();
}

bool FWindowsPlatformNamedPipe::ReadBytes(int32 NumBytes, void* OutData)
{
	check(NumBytes > 0);
	switch (State)
	{
		case State_Created:
			UE_LOGF(LogWindows, Fatal, "Need to OpenConnection() first!!\n");
			check(0);
			break;

		case State_ReadyForRW:
			// Nothing to do here, carry on
			break;

		case State_Uninitialized:
			UE_LOGF(LogWindows, Fatal, "Need to Create() first!!\n");
			check(0);
			break;

		case State_WaitingForRW:
		case State_Connecting:
			// Need to wait outside the client!
			check(0);
			break;

		case State_ErrorPipeClosedUnexpectedly:
			return false;

		default:
			// Invalid state!
			check(0);
			break;
	}


	//FPlatformMisc::LowLevelOutputDebugStringf(*FString::Printf(TEXT("*** %s: Reading %d bytes\n"), *Name, NumBytes));

	Windows::DWORD ReadBytes = 0;
	Windows::LPDWORD ReadBytesPtr = bUseOverlapped ? nullptr : &ReadBytes;
	BOOL Result = Windows::ReadFile(Pipe, OutData, NumBytes, ReadBytesPtr, bUseOverlapped ? &Overlapped : nullptr);
	if (!bUseOverlapped)
	{
		//FPlatformMisc::LowLevelOutputDebugStringf(*FString::Printf(TEXT("\t\tActually read %d bytes\n"), ReadBytes));
		if (!Result)
		{
			uint32 LastError = GetLastError();
			switch (LastError)
			{
				case ERROR_BROKEN_PIPE:
				case ERROR_PIPE_NOT_CONNECTED:
					State = State_ErrorPipeClosedUnexpectedly;
					LogInternals("ReadBytes=ErrorPipeClosedUnexpectedly");
					break;

				default:
					VerifyWinResult(0, TEXT("During ReadBytes()"));
					break;
			}
		}

		return(Result && NumBytes == ReadBytes);
	}

	if (Result)
	{
		// Operation finished
		State = State_ReadyForRW;
		LogInternals("ReadBytes=ReadyForRW");
		return true;
	}

	if (!UpdateAsyncStatusAfterRW())
	{
		return false;
	}

	State = State_WaitingForRW;
	LogInternals("ReadBytes=WaitingForRW");
	return UpdateAsyncStatus();
}

bool FWindowsPlatformNamedPipe::IsCreated() const
{
	switch (State)
	{
		case State_Created:
		case State_ReadyForRW:
		case State_WaitingForRW:
		case State_Connecting:
			return true;

		case State_Uninitialized:
		case State_ErrorPipeClosedUnexpectedly:
			break;

		default:	
			// Invalid state!
			check(0);
			break;
	}

	return false;
}

bool FWindowsPlatformNamedPipe::HasFailed() const
{
	switch (State)
	{
		case State_ErrorPipeClosedUnexpectedly:
			return true;

		default:	
			break;
	}

	return false;
}

void FWindowsPlatformNamedPipe::LogInternals(const char* Context) const
{
	if (CVarWindowsLogNamedPipeEvents.GetValueOnAnyThread())
	{
		UE_LOGF(LogWindows, Log, "[%s] Name=%ls Pipe=0x%llX Overlapped=%llX:%llX:%llX:%llX bUseOverlapped=%d bIsServer=%d State=%d",
			Context,
			NamePtr ? **NamePtr : TEXT("[Unknown]"),
			(unsigned long long)Pipe,
			Overlapped.Internal,
			Overlapped.InternalHigh,
			(unsigned long long)Overlapped.Pointer,
			(unsigned long long)Overlapped.hEvent,
			(int32)bUseOverlapped,
			(int32)bIsServer,
			(int32)State);
	}
}

#endif // PLATFORM_SUPPORTS_NAMED_PIPES
