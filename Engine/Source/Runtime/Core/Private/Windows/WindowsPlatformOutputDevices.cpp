// Copyright Epic Games, Inc. All Rights Reserved.

#include "Windows/WindowsPlatformOutputDevices.h"
#include "Logging/LogMacros.h"
#include "Misc/OutputDevice.h"
#include "HAL/PlatformTime.h"
#include "Math/UnrealMathUtility.h"
#include "HAL/UnrealMemory.h"
#include "Templates/UnrealTemplate.h"
#include "CoreGlobals.h"
#include "Misc/CString.h"
#include "Misc/Parse.h"
#include "Misc/CommandLine.h"
#include "Internationalization/Internationalization.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/OutputDeviceHelper.h"
#include "Misc/CoreDelegates.h"
#include "Misc/App.h"
#include "HAL/FeedbackContextAnsi.h"
#include "Windows/WindowsErrorOutputDevice.h"
#include "Windows/WindowsEventLogOutputDevice.h"
#include "HAL/ThreadHeartBeat.h"

//////////////////////////////////
// FWindowsPlatformOutputDevices
//////////////////////////////////

#ifndef WANTS_WINDOWS_EVENT_LOGGING
	#if UE_SERVER
		#define WANTS_WINDOWS_EVENT_LOGGING 1
	#else
		#define WANTS_WINDOWS_EVENT_LOGGING 0
	#endif
#endif

class FOutputDevice* FWindowsPlatformOutputDevices::GetEventLog()
{
#if WANTS_WINDOWS_EVENT_LOGGING
	static FWindowsEventLogOutputDevice Singleton;
	return &Singleton;
#else // no event logging
	return NULL;
#endif //WANTS_WINDOWS_EVENT_LOGGING
}

FOutputDeviceError* FWindowsPlatformOutputDevices::GetError()
{
	static FWindowsErrorOutputDevice Singleton;
	return &Singleton;
}

