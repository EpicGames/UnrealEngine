// Copyright Epic Games, Inc. All Rights Reserved.

#include "Windows/WindowsPlatformPerfCounters.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "ProfilingDebugging/CountersTrace.h"
#include "Templates/Function.h"
#include "HAL/PlatformAffinity.h"
#include "HAL/PlatformProcess.h"
#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "Misc/CommandLine.h"
#include "Misc/Parse.h"

#if COUNTERSTRACE_ENABLED

#pragma comment(lib, "pdh")

// Fix for older Windows Kits
// error C2040 : 'HLOG' : '_HLOG' differs in levels of indirection from 'PDH_HLOG'
#ifndef _LMHLOGDEFINED_
#define _LMHLOGDEFINED_

typedef struct _HLOG {
	DWORD          time;
	DWORD          last_flags;
	DWORD          offset;
	DWORD          rec_offset;
} HLOG, * PHLOG, * LPHLOG;

#endif

#include "Windows/AllowWindowsPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#include <pdh.h>
#include <pdhmsg.h>
THIRD_PARTY_INCLUDES_END
#include "Windows/HideWindowsPlatformTypes.h"

TRACE_DECLARE_MEMORY_COUNTER(Memory_TotalVirtual,					TEXT("PC / Memory / Total Virtual"));
TRACE_DECLARE_MEMORY_COUNTER(Memory_TotalPhysical,					TEXT("PC / Memory / Total Physical"));
TRACE_DECLARE_MEMORY_COUNTER(Memory_UsedPhysical,					TEXT("PC / Memory / Used Physical (Main Process)"));
TRACE_DECLARE_MEMORY_COUNTER(Memory_UsedVirtual,					TEXT("PC / Memory / Used Virtual (Main Process)"));
TRACE_DECLARE_MEMORY_COUNTER(Memory_AvailablePhysical,				TEXT("PC / Memory / Available Physical"));
TRACE_DECLARE_MEMORY_COUNTER(Memory_AvailableVirtual,				TEXT("PC / Memory / Available Virtual"));
TRACE_DECLARE_MEMORY_COUNTER(Memory_FreeAndZeroPageList,			TEXT("PC / Memory / Free & Zero Page List"));
TRACE_DECLARE_MEMORY_COUNTER(Memory_PrivateBytesShaderCompilation,	TEXT("PC / Memory / Private Bytes (Local Shader Compile Workers)"));
TRACE_DECLARE_MEMORY_COUNTER(Memory_UsedVirtualShaderCompilation,	TEXT("PC / Memory / Used Virtual (Local Shader Compile Workers)"));
TRACE_DECLARE_FLOAT_COUNTER(System_DemandZeroPageFaults,			TEXT("PC / Memory / Demand Zero Page Faults"));
TRACE_DECLARE_FLOAT_COUNTER(System_GPU_Usage,						TEXT("PC / GPU Engine / Total Usage"));
TRACE_DECLARE_MEMORY_COUNTER(Memory_UsedDedicatedVRAM,				TEXT("PC / GPU Memory / Used Dedicated"));
TRACE_DECLARE_FLOAT_COUNTER(System_CPU_Usage,						TEXT("PC / CPU / Total Usage"));
TRACE_DECLARE_INT_COUNTER(System_Thread_Count,						TEXT("PC / CPU / Total Thread Count"));
TRACE_DECLARE_INT_COUNTER(System_Network_Sent,						TEXT("PC / Network / Total Bytes Sent"));
TRACE_DECLARE_INT_COUNTER(System_Network_Received,					TEXT("PC / Network / Total Bytes Received"));

namespace WindowsPlatformPerfCountersImpl 
{
	
class FPerfCountersThread : public FRunnable
{
public:
	FPerfCountersThread(float SampleIntervalInSeconds)
		: SampleIntervalInSeconds(SampleIntervalInSeconds)
	{
		Thread = FRunnableThread::Create(this, TEXT("PerfCountersThread"), 512 * 1024, TPri_AboveNormal);
		check(Thread);
	}

	~FPerfCountersThread()
	{
		bContinue = false;
		delete Thread;
		Thread = nullptr;
	}

	struct FCounter
	{
		LPCTSTR Name;
		TFunction<void(double Value)> Setter;
		PDH_HCOUNTER Handle = nullptr;
	};

	/** Attaches to the task graph stats thread, all processing will be handled by the task graph. */
	virtual uint32 Run() override
	{
		FCounter Counters[] =
		{
			{ L"\\Processor(_Total)\\% Processor Time",				[](double Value) { TRACE_COUNTER_SET(System_CPU_Usage, Value); } },
			{ L"\\Process(_Total)\\Thread Count",					[](double Value) { TRACE_COUNTER_SET(System_Thread_Count, (int64)Value); } },
			{ L"\\Memory\\Demand Zero Faults/sec",					[](double Value) { TRACE_COUNTER_SET(System_DemandZeroPageFaults, Value); } },
			{ L"\\Memory\\Free & Zero Page List Bytes",				[](double Value) { TRACE_COUNTER_SET(Memory_FreeAndZeroPageList, (int64)Value); } },
			{ L"\\Network Interface(*)\\Bytes Received/sec",		[](double Value) { TRACE_COUNTER_SET(System_Network_Received, (int64)Value); } },
			{ L"\\Network Interface(*)\\Bytes Sent/sec",			[](double Value) { TRACE_COUNTER_SET(System_Network_Sent, (int64)Value); } },
			{ L"\\GPU Engine(*)\\Utilization Percentage",			[](double Value) { TRACE_COUNTER_SET(System_GPU_Usage, Value); } },
			{ L"\\GPU Adapter Memory(*)\\Dedicated Usage",			[](double Value) { TRACE_COUNTER_SET(Memory_UsedDedicatedVRAM, (int64)Value); } },
			{ L"\\Process(ShaderCompileWorker*)\\Page File Bytes",	[](double Value) { TRACE_COUNTER_SET(Memory_UsedVirtualShaderCompilation, (int64)Value); } },
			{ L"\\Process(ShaderCompileWorker*)\\Private Bytes",	[](double Value) { TRACE_COUNTER_SET(Memory_PrivateBytesShaderCompilation, (int64)Value); } },
		};

		PDH_HQUERY PdhQueryHandle = nullptr;
		PdhOpenQuery(NULL, 0, &PdhQueryHandle);

		for (FCounter& Counter : Counters)
		{
			PdhAddCounter(PdhQueryHandle, Counter.Name, 0, &Counter.Handle);
		}

		DWORD CurrentBufferSize = 0;
		PDH_FMT_COUNTERVALUE_ITEM* Buffer = nullptr;
		const float SampleInterval = this->SampleIntervalInSeconds;

		while (bContinue)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FCountersStatsThread::Tick);

			FPlatformProcess::SleepNoStats(SampleInterval);

			FPlatformMemoryStats Stats = FPlatformMemory::GetStats();
			TRACE_COUNTER_SET(Memory_TotalPhysical, Stats.TotalPhysical);
			TRACE_COUNTER_SET(Memory_TotalVirtual, Stats.TotalVirtual);
			TRACE_COUNTER_SET(Memory_UsedPhysical, Stats.UsedPhysical);
			TRACE_COUNTER_SET(Memory_UsedVirtual, Stats.UsedVirtual);
			TRACE_COUNTER_SET(Memory_AvailablePhysical, Stats.AvailablePhysical);
			TRACE_COUNTER_SET(Memory_AvailableVirtual, Stats.AvailableVirtual);

			PdhCollectQueryData(PdhQueryHandle);

			for (FCounter& Counter : Counters)
			{
				DWORD ItemCount = 0;
				DWORD RequiredBufferSize = 0;

				PDH_STATUS PdhStatus = PdhGetFormattedCounterArray(Counter.Handle, PDH_FMT_DOUBLE, &RequiredBufferSize, &ItemCount, nullptr);
				if (PDH_MORE_DATA == PdhStatus)
				{
					if (Buffer == nullptr || RequiredBufferSize > CurrentBufferSize)
					{
						Buffer = (PDH_FMT_COUNTERVALUE_ITEM*)FMemory::Realloc(Buffer, RequiredBufferSize);
						CurrentBufferSize = RequiredBufferSize;
					}

					if (Buffer)
					{
						PdhStatus = PdhGetFormattedCounterArray(Counter.Handle, PDH_FMT_DOUBLE, &RequiredBufferSize, &ItemCount, Buffer);
						if (ERROR_SUCCESS == PdhStatus)
						{
							double doubleValue = 0;

							for (DWORD Index = 0; Index < ItemCount; ++Index)
							{
								doubleValue += Buffer[Index].FmtValue.doubleValue;
							}

							Counter.Setter(doubleValue);
						}
					}
				}
			}
		}

		FMemory::Free(Buffer);
		PdhCloseQuery(PdhQueryHandle);

		return 0;
	}
	std::atomic_bool bContinue{ true };
	FRunnableThread* Thread = nullptr;
	float SampleIntervalInSeconds;
};

static FPerfCountersThread* PerfCountersThread = nullptr;
}

#endif // #if COUNTERSTRACE_ENABLED

void FWindowsPlatformPerfCounters::Init()
{
#if COUNTERSTRACE_ENABLED
	using namespace WindowsPlatformPerfCountersImpl;
	if (PerfCountersThread == nullptr && FParse::Param(FCommandLine::Get(), TEXT("perfcounters")))
	{
		float SampleInterval = 0;
		FParse::Value(FCommandLine::Get(), TEXT("perfcountersinterval="), SampleInterval);
		if (SampleInterval <= 0)
		{
			SampleInterval = 0.25f;
		}
		PerfCountersThread = new FPerfCountersThread(SampleInterval);
	}
#endif
}

void FWindowsPlatformPerfCounters::Shutdown()
{
#if COUNTERSTRACE_ENABLED
	using namespace WindowsPlatformPerfCountersImpl;

	delete PerfCountersThread;
	PerfCountersThread = nullptr;
#endif
}
