// Copyright Epic Games, Inc. All Rights Reserved.

#include "Windows/WindowsPlatformProcess.h"

#include "Containers/Set.h"
#include "Containers/StringConv.h"
#include "Containers/UnrealString.h"
#include "Async/Fundamental/Oversubscription.h"
#include "CoreGlobals.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/Platform.h"
#include "HAL/PlatformAffinity.h"
#include "HAL/PlatformMisc.h"
#include "HAL/FileManager.h"
#include "HAL/UnrealMemory.h"
#include "Internationalization/Internationalization.h"
#include "Logging/LogMacros.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreDelegates.h"
#include "Misc/CoreStats.h"
#include "Misc/Fork.h"
#include "Misc/Parse.h"
#include "Misc/Paths.h"
#include "Misc/ScopeExit.h"
#include "Misc/SingleThreadEvent.h"
#include "Misc/PathViews.h"
#include "Misc/TrackedActivity.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "Stats/Stats.h"
#include "Stats/ThreadIdleStats.h"
#include "Templates/UnrealTemplate.h"
#include "Trace/Trace.h"
#include "Trace/Trace.inl"

#include "Windows/AllowWindowsPlatformTypes.h"
	#include <shellapi.h>
	#include <ShlObj.h>
	#include <Shlwapi.h>
	#include <LM.h>
	#include <Psapi.h>
	#include <TlHelp32.h>
#include "Windows/HideWindowsPlatformTypes.h"
#include "Windows/WindowsPlatformMisc.h"

#pragma comment(lib, "psapi.lib")

#ifndef WINDOWS_ALLOW_BASEDIR_OVERRIDE
#define WINDOWS_ALLOW_BASEDIR_OVERRIDE 1
#endif

PRAGMA_DISABLE_UNSAFE_TYPECAST_WARNINGS

// static variables
FLaunchURLOverride FWindowsPlatformProcess::LaunchURLOverrideDelegate;
TArray<FString> FWindowsPlatformProcess::DllDirectoryStack;
TArray<FString> FWindowsPlatformProcess::DllDirectories;
TMap<FName, TArray<FString>> FWindowsPlatformProcess::SearchPathDllCache;
bool IsJobObjectSet = false;
HANDLE GhJob = NULL;

namespace WindowsPlatformProcess
{
	/**
	 * Maintain a named mutex to detect whether we are the first instance of this game
	 */
	static HANDLE GNamedMutex = NULL;

	void ReleaseNamedMutex(void)
	{
		if (GNamedMutex)
		{
			ReleaseMutex(GNamedMutex);
			GNamedMutex = NULL;
		}
	}

	bool MakeNamedMutex(const TCHAR* CmdLine)
	{
		bool bIsFirstInstance = false;

		TCHAR MutexName[MAX_SPRINTF] = TEXT("");

		FCString::Strncpy(MutexName, TEXT("UnrealEngine4"), MAX_SPRINTF);

		GNamedMutex = CreateMutex(NULL, true, MutexName);

		if (GNamedMutex && GetLastError() != ERROR_ALREADY_EXISTS && !FParse::Param(CmdLine, TEXT("NEVERFIRST")))
		{
			// We're the first instance!
			bIsFirstInstance = true;
		}
		else
		{
			// Still need to release it in this case, because it gave us a valid copy
			ReleaseNamedMutex();
			// There is already another instance of the game running.
			bIsFirstInstance = false;
		}

		return(bIsFirstInstance);
	}
}

/**
 * Checks if a given handle can be inherited by a child process.
 * 
 * @param Handle	The handle to check
 * @param Label		A descriptor used to improve logging if an error is encountered.
 * 
 * @return If the handle is valid and can inherited (has the HANDLE_FLAG_INHERIT flag)
 * then this returns the handle, otherwise it will return nullptr.
 */
static void* VerifyHandleIsInheritable(void* Handle, const ANSICHAR* Label)
{
	if (!Handle)
	{
		return nullptr;
	}

	DWORD Flags = 0;
	ensure(GetHandleInformation(HANDLE(Handle), &Flags));
	if (!(Flags & HANDLE_FLAG_INHERIT))
	{
		UE_LOGF(LogWindows, Error, "%s passed to CreateProc is not inheritable.", Label);
		return nullptr;
	}

	return Handle;
}

static bool bCustomProcessAffinity = false;
bool FWindowsPlatformProcess::SetProcessAffinity(uint32 NumCoresForAffinity, bool bPhysicalCoresOnly)
{
	if (NumCoresForAffinity == 0)
	{
		return false;
	}

	DWORD_PTR AffinityMask = 0;
	if (!bPhysicalCoresOnly)
	{
		if (NumCoresForAffinity > 64)
		{
			UE_LOGF(LogWindows, Warning, "Requested process affinity to %d logical cores but the maximum affinity is 64 cores. Will use 64.", NumCoresForAffinity);
			NumCoresForAffinity = 64;
		}

		if (NumCoresForAffinity == 64)
		{
			AffinityMask = 0xFFFFFFFFFFFFFFFF;
		}
		else
		{
			AffinityMask = (((DWORD_PTR)1) << NumCoresForAffinity) - 1;
		}
	}
	else
	{
		if (NumCoresForAffinity > 32)
		{
			UE_LOGF(LogWindows, Warning, "Requested process affinity to %d physical cores but the maximum affinity is 32 cores. Will use 32.", NumCoresForAffinity);
			NumCoresForAffinity = 32;
		}

		// Windows numbers physical and logical (hyperthreaded) cores by interleaving them.
		// So 0 is physical, 1 is logical, 2 is physical, 3 is logical, etc.
		constexpr DWORD_PTR PhysicalMask = 0x5555555555555555;

		if (NumCoresForAffinity == 32)
		{
			AffinityMask = PhysicalMask;
		}
		else
		{
			AffinityMask = (((DWORD_PTR)1) << (NumCoresForAffinity * 2)) - 1;
			AffinityMask &= PhysicalMask;
		}
	}


	if (!SetProcessAffinityMask(GetCurrentProcess(), AffinityMask))
	{
		DWORD LastError = GetLastError();
		TCHAR ErrorMsg[1024];
		FPlatformMisc::GetSystemErrorMessage(ErrorMsg, 1024, LastError);
		UE_LOGF(LogWindows, Error, "Failed to set process affinity, process may run on all available logical cores. Error: %d [%ls]", LastError, ErrorMsg);
	}
	else
	{
		UE_LOGF(LogWindows, Log, "Successfully set process affinity, process will only use %d cores", NumCoresForAffinity);
		bCustomProcessAffinity = true;
	}

	return bCustomProcessAffinity;
}

bool FWindowsPlatformProcess::IsProcessAffinitySet()
{
	return bCustomProcessAffinity;
}

void FWindowsPlatformProcess::AddDllDirectory(const TCHAR* Directory)
{
	FString NormalizedDirectory = FPaths::ConvertRelativePathToFull(Directory);
	FPaths::NormalizeDirectoryName(NormalizedDirectory);
	FPaths::MakePlatformFilename(NormalizedDirectory);

	if (DllDirectories.Find(NormalizedDirectory) == INDEX_NONE)
	{
		DllDirectories.Add(NormalizedDirectory);

		// enumerate the dir and cache all the dlls
		{
			TArray<FString> FoundDllFileNames;
			IPlatformFile::GetPlatformPhysical().FindFiles(FoundDllFileNames, *NormalizedDirectory, TEXT(".dll"));
			for (FString& DllPath : FoundDllFileNames)
			{
				TArray<FString>& Paths = SearchPathDllCache.FindOrAdd(FName(FPathViews::GetCleanFilename(DllPath)));
				FPaths::NormalizeDirectoryName(DllPath);
				Paths.Add(DllPath);
			}
		}
	}
}

void FWindowsPlatformProcess::GetDllDirectories(TArray<FString>& OutDllDirectories)
{
	OutDllDirectories = DllDirectories;
}

void* FWindowsPlatformProcess::GetDllHandle( const TCHAR* FileName )
{
	check(FileName);

	// Combine the explicit DLL search directories with the contents of the directory stack
	// Note that the search path logic here needs to match the logic found in ResolveImport
	// Use an inline allocator because this can be called from assertions within the
	// allocator and the array allocations may fail in that situation.
	TArray<FString, TInlineAllocator<16>> SearchPaths;
	SearchPaths.Reserve(1 + ((DllDirectoryStack.Num() > 0) ? 1 : 0) + DllDirectories.Num());
	SearchPaths.Add(FPlatformProcess::GetModulesDirectory());
	if(DllDirectoryStack.Num() > 0)
	{
		SearchPaths.Add(DllDirectoryStack.Top());
	}
	for(int32 Idx = 0; Idx < DllDirectories.Num(); Idx++)
	{
		SearchPaths.Add(DllDirectories[Idx]);
	}

	// Load the DLL, avoiding windows dialog boxes if missing
	static const bool CMDLINE_dllerrors = FParse::Param(::GetCommandLineW(), TEXT("dllerrors"));
	static const bool CMDLINE_unattended = FParse::Param(::GetCommandLineW(), TEXT("unattended"));
	
	DWORD ErrorMode = 0;
	if(!CMDLINE_dllerrors)
	{
		ErrorMode |= SEM_NOOPENFILEERRORBOX;
		if(CMDLINE_unattended)
		{
			ErrorMode |= SEM_FAILCRITICALERRORS | SEM_NOGPFAULTERRORBOX;
		}
	}

	DWORD PrevErrorMode = 0;
	const BOOL bHavePrevErrorMode = ::SetThreadErrorMode(ErrorMode, &PrevErrorMode);

	// Load the DLL, avoiding windows dialog boxes if missing
	void* Handle = LoadLibraryWithSearchPaths(FileName, SearchPaths);
	
	if(bHavePrevErrorMode)
	{
		::SetThreadErrorMode(PrevErrorMode, NULL);
	}

	return Handle;
}

void FWindowsPlatformProcess::FreeDllHandle( void* DllHandle )
{
	// It is okay to call FreeLibrary on 0
	::FreeLibrary((HMODULE)DllHandle);
}

FString FWindowsPlatformProcess::GenerateApplicationPath( const FString& AppName, EBuildConfiguration BuildConfiguration)
{
	FString PlatformName = GetBinariesSubdirectory();
	FString ExecutablePath = FPaths::EngineDir() / FString::Printf(TEXT("Binaries/%s/%s"), *PlatformName, *AppName);
	FPaths::MakePlatformFilename(ExecutablePath);

	if (BuildConfiguration != EBuildConfiguration::Development)
	{
		ExecutablePath += FString::Printf(TEXT("-%s-%s"), *PlatformName, LexToString(BuildConfiguration));
	}

	ExecutablePath += TEXT(".exe");

	return ExecutablePath;
}

const TCHAR* FWindowsPlatformProcess::GetArchitectureSuffix()
{
#if PLATFORM_WINDOWS_ARM64EC
	return TEXT("arm64ec");
#elif PLATFORM_CPU_ARM_FAMILY
	return TEXT("arm64");
#else
	return nullptr;
#endif
}

void* FWindowsPlatformProcess::GetDllExport( void* DllHandle, const TCHAR* ProcName )
{
	check(DllHandle);
	check(ProcName);
	return (void*)::GetProcAddress( (HMODULE)DllHandle, TCHAR_TO_ANSI(ProcName) );
}

void FWindowsPlatformProcess::PushDllDirectory(const TCHAR* Directory)
{
	// set the directory in windows
	::SetDllDirectory(Directory);
	// remember it
	DllDirectoryStack.Push(Directory);
}

void FWindowsPlatformProcess::PopDllDirectory(const TCHAR* Directory)
{
	// don't allow too many pops (indicates bad code that should be fixed, but won't kill anything, so using ensure)
	ensureMsgf(DllDirectoryStack.Num() > 0, TEXT("Tried to PopDllDirectory too many times"));
	// verify we are popping the top
	checkf(DllDirectoryStack.Top() == Directory, TEXT("There was a PushDllDirectory/PopDllDirectory mismatch (Popped %s, which didn't match %s)"), *DllDirectoryStack.Top(), Directory);
	// pop it off
	DllDirectoryStack.Pop();

	// and now set the new DllDirectory to the old value
	if (DllDirectoryStack.Num() > 0)
	{
		::SetDllDirectory(*DllDirectoryStack.Top());
	}
	else
	{
		::SetDllDirectory(TEXT(""));
	}
}

static void LaunchWebURL( const FString& URLParams, FString* Error )
{
	UE_LOGF(LogWindows, Log, "LaunchURL %ls", *URLParams);

	FString BrowserOpenCommand;
	BrowserOpenCommand.GetCharArray().SetNumUninitialized(MAX_PATH);

	// Look up the shell open command for the http protocol
	bool bRetry = true;
	for (;;)
	{
		// Query the command
		DWORD Size = BrowserOpenCommand.GetCharArray().Num();
		HRESULT hResult = AssocQueryString(ASSOCF_NOTRUNCATE | ASSOCF_IS_PROTOCOL, ASSOCSTR_COMMAND, TEXT("http"), TEXT("open"), BrowserOpenCommand.GetCharArray().GetData(), &Size);

		// Retry once if the buffer isn't big enough
		if (hResult == E_POINTER || hResult == S_FALSE || hResult == HRESULT_FROM_WIN32(ERROR_MORE_DATA))
		{
			if (bRetry && Size <= (DWORD)MIN_int32)
			{
				bRetry = false;
				BrowserOpenCommand.GetCharArray().SetNum((int32)Size);
				continue;
			}
		}

		// If there's still an error, assume the command couldn't be found
		if (hResult != S_OK)
		{
			Size = 0;
		}

		// Resize the string to the right size
		BrowserOpenCommand.GetCharArray().SetNum((int32)Size);
		break;
	}

	// If we have successfully looked up the correct shell command, then we can create a new process using that command
	// we do this instead of shell execute due to security concerns.  By starting the browser directly we avoid most issues.
	if (!BrowserOpenCommand.IsEmpty())
	{
		FString ExePath, ExeArgs;

		// If everything has gone to plan, the shell command should be something like this:
		// "C:\Program Files (x86)\Mozilla Firefox\firefox.exe" -osint -url "%1"
		// We need to extract out the executable portion, and the arguments portion and expand any %1's with the URL,
		// then start the browser process.

		// Extract the exe and any arguments to the executable.
		const int32 FirstQuote = BrowserOpenCommand.Find(TEXT("\""));
		if (FirstQuote != INDEX_NONE)
		{
			const int32 SecondQuote = BrowserOpenCommand.Find(TEXT("\""), ESearchCase::IgnoreCase, ESearchDir::FromStart, FirstQuote + 1);
			if (SecondQuote != INDEX_NONE)
			{
				ExePath = BrowserOpenCommand.Mid(FirstQuote + 1, (SecondQuote - 1) - FirstQuote);
				ExeArgs = BrowserOpenCommand.Mid(SecondQuote + 1);
			}
		}

		// If anything failed to parse right, don't continue down this path, just use shell execute.
		if (!ExePath.IsEmpty())
		{
			if (ExeArgs.ReplaceInline(TEXT("%1"), *URLParams) == 0)
			{
				// If we fail to detect the placement token we append the URL to the arguments.
				// This is for robustness, and to fix a known error case when using Internet Explorer 8. 
				ExeArgs.Append(TEXT(" \"") + URLParams + TEXT("\""));
			}

			// Now that we have the shell open command to use, run the shell command in the open process with any and all parameters.
			if (FPlatformProcess::CreateProc(*ExePath, *ExeArgs, true, false, false, NULL, 0, NULL, NULL).IsValid())
			{
				// Success!
				return;
			}
			else
			{
				if (Error)
				{
					*Error = NSLOCTEXT("Core", "UrlFailed", "Failed launching URL").ToString();
				}
			}
		}
	}

	// If all else fails just do a shell execute and let windows sort it out.  But only do it if it's an
	// HTTP or HTTPS address.  A malicious address could be problematic if just passed directly to shell execute.
	if (URLParams.StartsWith(TEXT("http://")) || URLParams.StartsWith(TEXT("https://")))
	{
		const HINSTANCE Code = ::ShellExecuteW(NULL, TEXT("open"), *URLParams, NULL, NULL, SW_SHOWNORMAL);
		if (Error)
		{
			*Error = ((PTRINT)Code <= 32) ? NSLOCTEXT("Core", "UrlFailed", "Failed launching URL").ToString() : TEXT("");
		}
	}
}

static void LaunchDefaultHandlerForURL( const TCHAR* URL, FString* Error )
{
	// ShellExecute will open the default handler for a URL
	const HINSTANCE Code = ::ShellExecuteW(NULL, TEXT("open"), URL, NULL, NULL, SW_SHOWNORMAL);
	if (Error)
	{
		*Error = ((PTRINT)Code <= 32) ? NSLOCTEXT("Core", "UrlFailed", "Failed launching URL").ToString() : TEXT("");
	}
}

bool FWindowsPlatformProcess::CanLaunchURL(const TCHAR* URL)
{
	return URL != nullptr;
}

void FWindowsPlatformProcess::LaunchURL(const TCHAR* URL, const TCHAR* Parms, FString* Error)
{
	check(URL);

	if (LaunchURLOverrideDelegate.IsBound())
	{
		LaunchURLOverrideDelegate.Execute(URL);
	}
	else
	{
		LaunchURLInternal(URL, Parms, Error);
	}
}

void FWindowsPlatformProcess::LaunchURLInternal(const TCHAR* URL, const TCHAR* Parms, FString* Error)
{
	if (FCoreDelegates::ShouldLaunchUrl.IsBound() && !FCoreDelegates::ShouldLaunchUrl.Execute(URL))
	{
		if (Error)
		{
			*Error = TEXT("LaunchURL cancelled by delegate");
		}
	}
	else
	{
		// Initialize the error to empty string.
		if (Error)
		{
			*Error = TEXT("");
		}

		if (FCoreDelegates::LaunchCustomHandlerForURL.IsBound())
		{
			FCoreDelegates::LaunchCustomHandlerForURL.Execute(URL, Error);
		}
		// Use the default handler if we have a URI scheme name that doesn't look like a Windows path, and is not http: or https:
		else if (FString SchemeName; FParse::SchemeNameFromURI(URL, SchemeName) && SchemeName.Len() > 1 && SchemeName != TEXT("http") && SchemeName != TEXT("https"))
		{
			LaunchDefaultHandlerForURL(URL, Error);
		}
		else
		{
			FString URLParams = FString::Printf(TEXT("%s %s"), URL, Parms ? Parms : TEXT("")).TrimEnd();
			LaunchWebURL(URLParams, Error);
		}
	}
}

FProcHandle FWindowsPlatformProcess::CreateProc(const TCHAR* URL, const TCHAR* Parms, bool bLaunchDetached, bool bLaunchHidden, bool bLaunchReallyHidden, uint32* OutProcessID, int32 PriorityModifier, const TCHAR* OptionalWorkingDirectory, void* PipeWriteChild, void* PipeReadChild)
{
	PipeWriteChild	= VerifyHandleIsInheritable(PipeWriteChild, "PipeWriteChild");
	PipeReadChild	= VerifyHandleIsInheritable(PipeReadChild, "PipeReadChild");

	// CreateProc used to only have a single "write" pipe argument that both stdout and stderr would be piped into on Windows,
	// so for this overload we'll preserve that behaviour for compatibility with existing code
	return CreateProcInternal(URL, Parms, bLaunchDetached, bLaunchHidden, bLaunchReallyHidden, false, OutProcessID, PriorityModifier, OptionalWorkingDirectory, PipeWriteChild, PipeReadChild, PipeWriteChild, false);
}

FProcHandle FWindowsPlatformProcess::CreateProc(const TCHAR* URL, const TCHAR* Parms, bool bLaunchDetached, bool bLaunchHidden, bool bLaunchReallyHidden, uint32* OutProcessID, int32 PriorityModifier, const TCHAR* OptionalWorkingDirectory, void* PipeWriteChild, void* PipeReadChild, void* PipeStdErrChild)
{
	PipeWriteChild	= VerifyHandleIsInheritable(PipeWriteChild, "PipeWriteChild");
	PipeReadChild	= VerifyHandleIsInheritable(PipeReadChild, "PipeReadChild");
	PipeStdErrChild	= VerifyHandleIsInheritable(PipeStdErrChild, "PipeStdErrChild");

	return CreateProcInternal(URL, Parms, bLaunchDetached, bLaunchHidden, bLaunchReallyHidden, false, OutProcessID, PriorityModifier, OptionalWorkingDirectory, PipeWriteChild, PipeReadChild, PipeStdErrChild, false);
}

TTuple<FProcHandle, UE::HAL::EProcessId> FWindowsPlatformProcess::CreateProc(const UE::HAL::FProcessStartInfo& StartInfo)
{
	uint32 Id;
	FProcHandle Handle = CreateProcInternal(StartInfo.Uri, StartInfo.Arguments, StartInfo.bDetached, false, StartInfo.bHidden, StartInfo.bBreakawayFromJob, &Id, StartInfo.PriorityModifier, StartInfo.WorkingDirectory, StartInfo.StdOut, StartInfo.StdIn, StartInfo.StdErr, StartInfo.bInheritHandles);
	return {Handle, UE::HAL::EProcessId{Id}};
}

#include "Windows/AllowWindowsPlatformTypes.h"
FProcHandle FWindowsPlatformProcess::CreateProcInternal(const TCHAR* URL, const TCHAR* Parms, bool bLaunchDetached, bool bLaunchHidden, bool bLaunchReallyHidden, bool bBreakawayFromJob, uint32* OutProcessID, int32 PriorityModifier, const TCHAR* OptionalWorkingDirectory, void* PipeWriteChild, void* PipeReadChild, void* PipeStdErrChild, bool bInheritHandles)
{
	check(URL);
	check(URL[0] != '\"');
	check(Parms);
	
	auto HandleError = [&URL, &Parms, &OutProcessID]
	{
#if !UE_BUILD_SHIPPING
		DWORD Error = GetLastError();
		TCHAR ErrorMessage[512];
		FWindowsPlatformMisc::GetSystemErrorMessage(ErrorMessage, 512, Error);

		UE_LOGF(LogWindows, Warning, "CreateProc failed: %ls (0x%08x)", ErrorMessage, Error);
		if (Error == ERROR_NOT_ENOUGH_MEMORY || Error == ERROR_OUTOFMEMORY)
		{
			// These errors are common enough that we want some available memory information
			FPlatformMemoryStats Stats = FPlatformMemory::GetStats();
			UE_LOGF(LogWindows, Warning, "Mem used: %.2f MB, OS Free %.2f MB", Stats.UsedPhysical / 1048576.0f, Stats.AvailablePhysical / 1048576.0f);
		}
		UE_LOGF(LogWindows, Warning, "URL: %ls %ls", URL, Parms);
		
#endif
		if (OutProcessID)
		{
			*OutProcessID = 0;
		}
	};
	
	// CREATE_NEW_PROCESS_GROUP for consistency with Unix-like platforms
	DWORD CreationFlags = CREATE_NEW_PROCESS_GROUP;
	
	if (PriorityModifier < 0)
	{
		CreationFlags |= PriorityModifier == -1 ? BELOW_NORMAL_PRIORITY_CLASS : IDLE_PRIORITY_CLASS;
	}
	else if (PriorityModifier > 0)
	{
		CreationFlags |= PriorityModifier == 1 ? ABOVE_NORMAL_PRIORITY_CLASS : HIGH_PRIORITY_CLASS;
	}
	else
	{
		CreationFlags |= NORMAL_PRIORITY_CLASS;
	}

	if (bLaunchDetached)
	{
		// CREATE_NO_WINDOW because DETACHED_PROCESS can cause console windows to appear if the child spawns a process
		// https://ikriv.com/dev/cpp/ConsoleProxy/flags has the best explanation
		CreationFlags |= bLaunchReallyHidden ? CREATE_NO_WINDOW : DETACHED_PROCESS;
	}

	if (bBreakawayFromJob)
	{
		CreationFlags |= CREATE_BREAKAWAY_FROM_JOB;
	}

	DWORD Flags;
	WORD ShowWindow;
	if (bLaunchReallyHidden)
	{
		Flags = STARTF_USESHOWWINDOW;
		ShowWindow = SW_HIDE;
	}
	else if (bLaunchHidden)
	{
		Flags = STARTF_USESHOWWINDOW;
		ShowWindow = SW_SHOWMINNOACTIVE;
	}
	else
	{
		Flags = 0;
		ShowWindow = 0;
	}

	// Required to exist until DeleteProcThreadAttributeList is called
	HANDLE Handles[3];
	LPPROC_THREAD_ATTRIBUTE_LIST AttributeList = nullptr;
	ON_SCOPE_EXIT
	{
		if (AttributeList)
		{
			DeleteProcThreadAttributeList(AttributeList);
		}
	};

	if (!bInheritHandles)
	{
		uint32 HandleCount = 0;
		if (PipeWriteChild)
		{
			Handles[HandleCount++] = HANDLE(PipeWriteChild);
		}
		if (PipeReadChild && PipeReadChild != PipeWriteChild)
		{
			Handles[HandleCount++] = HANDLE(PipeReadChild);
		}
		if (PipeStdErrChild && PipeStdErrChild != PipeWriteChild && PipeStdErrChild != PipeReadChild)
		{
			Handles[HandleCount++] = HANDLE(PipeStdErrChild);
		}
		
		if (HandleCount)
		{
			SIZE_T Size = 0;
			verify(!InitializeProcThreadAttributeList(nullptr, 1, 0, &Size));
			check(GetLastError() == ERROR_INSUFFICIENT_BUFFER);
			check(Size > 0);
			check(Size < 1024);
			// Doing what .NET does here for efficiency
			void* Buffer = _alloca(Size);

			if (!InitializeProcThreadAttributeList(LPPROC_THREAD_ATTRIBUTE_LIST(Buffer), 1, 0, &Size))
			{
				HandleError();
				return {};
			}

			AttributeList = LPPROC_THREAD_ATTRIBUTE_LIST(Buffer);

			if (!UpdateProcThreadAttribute(AttributeList, 0, PROC_THREAD_ATTRIBUTE_HANDLE_LIST, Handles, HandleCount * sizeof(HANDLE), nullptr, nullptr))
			{
				HandleError();
				return {};
			}

			CreationFlags |= EXTENDED_STARTUPINFO_PRESENT;
			Flags |= STARTF_USESTDHANDLES;
		}
	}

	FString CommandLine = FString::Printf(TEXT("\"%s\" %s"), URL, Parms);
	STARTUPINFOEX StartupInfoEx
	{
		{
			.cb = sizeof(STARTUPINFOEX),
			.dwFlags = Flags,
			.wShowWindow = ShowWindow,
			.hStdInput = HANDLE(PipeReadChild),
			.hStdOutput = HANDLE(PipeWriteChild),
			.hStdError = HANDLE(PipeStdErrChild)
		},
		AttributeList
	};
	PROCESS_INFORMATION ProcessInformation;

	if (!CreateProcess(nullptr, CommandLine.GetCharArray().GetData(), nullptr, nullptr, bInheritHandles || AttributeList, CreationFlags, nullptr, OptionalWorkingDirectory, &StartupInfoEx.StartupInfo, &ProcessInformation))
	{
		HandleError();
		return {};
	}

	CloseHandle(ProcessInformation.hThread);

	if (OutProcessID)
	{
		*OutProcessID = ProcessInformation.dwProcessId;
	}

	return FProcHandle{ProcessInformation.hProcess};
}
#include "Windows/HideWindowsPlatformTypes.h"

bool FWindowsPlatformProcess::SetProcPriority(FProcHandle& InProcHandle, int32 PriorityModifier)
{
	DWORD PriorityClass = NORMAL_PRIORITY_CLASS;
	if (PriorityModifier < 0)
	{
		PriorityClass = (PriorityModifier == -1) ? BELOW_NORMAL_PRIORITY_CLASS : IDLE_PRIORITY_CLASS;
	}
	else if (PriorityModifier > 0)
	{
		PriorityClass = (PriorityModifier == 1) ? ABOVE_NORMAL_PRIORITY_CLASS : HIGH_PRIORITY_CLASS;
	}

	if (InProcHandle.IsValid())
	{
		return SetPriorityClass(InProcHandle.Get(), PriorityClass);
	}
	return false;

}

FProcHandle FWindowsPlatformProcess::OpenProcess(uint32 ProcessID)
{
	return FProcHandle(::OpenProcess(PROCESS_ALL_ACCESS, 0, ProcessID));
}

bool FWindowsPlatformProcess::IsProcRunning( FProcHandle & ProcessHandle )
{
	bool bApplicationRunning = true;
	uint32 WaitResult = ::WaitForSingleObject(ProcessHandle.Get(), 0);
	if (WaitResult != WAIT_TIMEOUT)
	{
		bApplicationRunning = false;
	}
	return bApplicationRunning;
}

void FWindowsPlatformProcess::WaitForProc( FProcHandle & ProcessHandle )
{
	::WaitForSingleObject(ProcessHandle.Get(), INFINITE);
}

void FWindowsPlatformProcess::CloseProc(FProcHandle & ProcessHandle)
{
	if (ProcessHandle.IsValid())
	{
		::CloseHandle(ProcessHandle.Get());
		ProcessHandle.Reset();
	}
}

void FWindowsPlatformProcess::TerminateProc( FProcHandle & ProcessHandle, bool KillTree )
{
	if (KillTree)
	{
		TerminateProcTreeWithPredicate(ProcessHandle, [](uint32 ProcessId, const TCHAR* ApplicationName) { return true; });
	}
	else
	{
		TerminateProcess(ProcessHandle.Get(),0);
	}
}

static void TerminateProcTreeWithPredicateInternal(
	HANDLE ProcessHandle,
	TFunctionRef<bool(uint32 ProcessId, const TCHAR* ApplicationName)> Predicate,
	TSet<DWORD>& VisitedProcessIds)
{
	::DWORD ProcessId = ::GetProcessId(ProcessHandle);
	FString ProcessName = FPlatformProcess::GetApplicationName(ProcessId);

	VisitedProcessIds.Add(ProcessId);

	if (!Predicate(ProcessId, *ProcessName))
	{
		return;
	}

	HANDLE SnapShot = ::CreateToolhelp32Snapshot(TH32CS_SNAPPROCESS, 0);

	if (SnapShot != INVALID_HANDLE_VALUE)
	{

		PROCESSENTRY32 Entry;
		Entry.dwSize = sizeof(PROCESSENTRY32);

		if (::Process32First(SnapShot, &Entry))
		{
			do
			{
				if (Entry.th32ParentProcessID == ProcessId)
				{
					HANDLE ChildProcHandle = ::OpenProcess(PROCESS_ALL_ACCESS, 0, Entry.th32ProcessID);

					if (ChildProcHandle)
					{
						if (!VisitedProcessIds.Contains(Entry.th32ProcessID))
						{
							TerminateProcTreeWithPredicateInternal(ChildProcHandle, Predicate, VisitedProcessIds);
						}
						::CloseHandle(ChildProcHandle);
					}
				}
			}
			while(::Process32Next(SnapShot, &Entry));
		}
	}
	::CloseHandle(SnapShot);

	TerminateProcess(ProcessHandle,0);
}

void FWindowsPlatformProcess::TerminateProcTreeWithPredicate(
	FProcHandle& ProcessHandle,
	TFunctionRef<bool(uint32 ProcessId, const TCHAR* ApplicationName)> Predicate)
{
	TSet<DWORD> VisitedProcessIds;
	TerminateProcTreeWithPredicateInternal(ProcessHandle.Get(), Predicate, VisitedProcessIds);
}

uint32 FWindowsPlatformProcess::GetCurrentProcessId()
{
	return ::GetCurrentProcessId();
}

uint32 FWindowsPlatformProcess::GetCurrentCoreNumber()
{
	return ::GetCurrentProcessorNumber();
}

void FWindowsPlatformProcess::SetThreadAffinityMask( uint64 AffinityMask )
{
	// While it's technically possible to use both process and thread affinities,
	// it requires restricting the set of cores eligible for affinity to respect 
	// the process affinity mask.
	// For simplicity, as long as the process-wide affinity is a debugging option,
	// disallow thread affinities when using process affinity.
	if (FWindowsPlatformProcess::IsProcessAffinitySet())
	{
		return;
	}

	if( AffinityMask != FPlatformAffinity::GetNoAffinityMask() )
	{
		::SetThreadAffinityMask( ::GetCurrentThread(), (DWORD_PTR)AffinityMask );
	}
}

bool FWindowsPlatformProcess::GetProcReturnCode( FProcHandle & ProcHandle, int32* ReturnCode )
{
	DWORD ExitCode = 0;
	if (::GetExitCodeProcess(ProcHandle.Get(), &ExitCode) && ExitCode != STILL_ACTIVE)
	{
		if (ReturnCode)
		{
			*ReturnCode = (int32)ExitCode;
		}
		return true;
	}
	return false;
}

bool FWindowsPlatformProcess::GetApplicationMemoryUsage(uint32 ProcessId, SIZE_T* OutMemoryUsage)
{
	bool bSuccess = false;
	HANDLE ProcessHandle = ::OpenProcess(PROCESS_QUERY_INFORMATION | PROCESS_VM_READ, false, ProcessId);

	if (ProcessHandle != NULL)
	{
		PROCESS_MEMORY_COUNTERS_EX MemoryInfo;

		if (GetProcessMemoryInfo(ProcessHandle, (PROCESS_MEMORY_COUNTERS*)&MemoryInfo, sizeof(MemoryInfo)))
		{
			*OutMemoryUsage = MemoryInfo.PrivateUsage;
			bSuccess = true;
		}

		::CloseHandle(ProcessHandle);
	}

	return bSuccess;
}

bool FWindowsPlatformProcess::GetPerFrameProcessorUsage(uint32 ProcessId, float& ProcessUsageFraction, float& IdleUsageFraction)
{
	bool bSuccess = true;

	static double LastProcessTime = 0.f;
	static double LastIdleTime = 0.f;
	static uint32 LastFrameNumber = 0;

	if (LastFrameNumber != GFrameNumber)
	{
		LastFrameNumber = GFrameNumber;

		// Get queryable process handle
		HANDLE ProcessHandle = ::OpenProcess(PROCESS_QUERY_INFORMATION | PROCESS_VM_READ, false, ProcessId);

		if (ProcessHandle != nullptr)
		{
			const uint64 NumCores = FPlatformMisc::NumberOfCoresIncludingHyperthreads();
			const uint32 CurrFrameIndex = LastFrameNumber % 2;
			const uint32 PrevFrameIndex = 1 - CurrFrameIndex;

			// Get total processor cycles per second
			static double DeltaCyclesPerSecond = 0.0;
			if (DeltaCyclesPerSecond == 0.0)
			{
				LARGE_INTEGER Frequency;
				QueryPerformanceFrequency(&Frequency);
				DeltaCyclesPerSecond = (double)(Frequency.QuadPart) * 1000.0;
				DeltaCyclesPerSecond *= NumCores;
			}

			// Calculate total number of cycles that have passed this frame
			static double PrevTotalSeconds = 0.0;
			double TotalSeconds = FPlatformTime::Seconds();
			double DeltaSecondsPerFrame = (double)(TotalSeconds - PrevTotalSeconds);
			PrevTotalSeconds = TotalSeconds;

			double DeltaCyclesPerFrame = DeltaSecondsPerFrame * DeltaCyclesPerSecond;

			// Grab cycle time for this process as fraction of total processor time
			static uint64 ProcessCycleTimeBuffers[2] = {0};
			uint64& ProcessCycleTime = ProcessCycleTimeBuffers[CurrFrameIndex];
			uint64& PrevProcessCycleTime = ProcessCycleTimeBuffers[PrevFrameIndex];

			if (!QueryProcessCycleTime(ProcessHandle, (PULONG64)&ProcessCycleTime))
			{
				bSuccess = false;
			}
			uint64 DeltaProcessCycleTime = ProcessCycleTime - PrevProcessCycleTime;
			LastProcessTime = (double)DeltaProcessCycleTime / DeltaCyclesPerFrame;

			// Idle cycles are stored per core and flipped to allow per-frame calculation
			const uint32 BufferLength = NumCores * 8;
			static uint64* IdleCycleTimeBuffers[2] = { nullptr };

			if(IdleCycleTimeBuffers[0] == nullptr)
			{
				//Alloc buffers on first frame 
				IdleCycleTimeBuffers[0] = new uint64[BufferLength];
				IdleCycleTimeBuffers[1] = new uint64[BufferLength];
			}

			uint64* IdleCycleTime = IdleCycleTimeBuffers[CurrFrameIndex];
			uint64* PrevIdleCycleTime = IdleCycleTimeBuffers[PrevFrameIndex];

			// Grab idle cycle time as percentage of total processor time
			// Note: Idle processes are specified per core and accumulated
			if (!QueryIdleProcessorCycleTime((PULONG)&BufferLength, (PULONG64)IdleCycleTime))
			{
				bSuccess = false;
			}

			uint64 DeltaIdleTime = 0;
			for (int Core = 0; Core < NumCores; ++Core)
			{
				DeltaIdleTime += IdleCycleTime[Core] - PrevIdleCycleTime[Core];
			}
			LastIdleTime = (double)DeltaIdleTime / DeltaCyclesPerFrame;

			::CloseHandle(ProcessHandle);
		}
		else
		{
			bSuccess = false;
		}
	}

	if (bSuccess)
	{
		ProcessUsageFraction = LastProcessTime * 100.0f;
		IdleUsageFraction = LastIdleTime * 100.0f;
	}
	else
	{
		ProcessUsageFraction = IdleUsageFraction = 0.f;
	}

	return bSuccess;
}

bool FWindowsPlatformProcess::IsApplicationRunning( uint32 ProcessId )
{
	bool bApplicationRunning = true;
	HANDLE ProcessHandle = ::OpenProcess(SYNCHRONIZE, false, ProcessId);
	if (ProcessHandle == NULL)
	{
		bApplicationRunning = false;
	}
	else
	{
		uint32 WaitResult = WaitForSingleObject(ProcessHandle, 0);
		if (WaitResult != WAIT_TIMEOUT)
		{
			bApplicationRunning = false;
		}
		::CloseHandle(ProcessHandle);
	}
	return bApplicationRunning;
}

bool FWindowsPlatformProcess::IsApplicationRunning( const TCHAR* ProcName )
{
	// append the extension

	FString ProcNameWithExtension = ProcName;
	if( ProcNameWithExtension.Find( TEXT(".exe"), ESearchCase::IgnoreCase, ESearchDir::FromEnd ) == INDEX_NONE )
	{
		ProcNameWithExtension += TEXT(".exe");
	}

	HANDLE SnapShot = ::CreateToolhelp32Snapshot( TH32CS_SNAPPROCESS, 0 );
	if( SnapShot != INVALID_HANDLE_VALUE )
	{
		PROCESSENTRY32 Entry;
		Entry.dwSize = sizeof( PROCESSENTRY32 );

		if( ::Process32First( SnapShot, &Entry ) )
		{
			do
			{
				if( FCString::Stricmp( *ProcNameWithExtension, Entry.szExeFile ) == 0 )
				{
					::CloseHandle( SnapShot );
					return true;
				}
			} while( ::Process32Next( SnapShot, &Entry ) );
		}
	}

	::CloseHandle( SnapShot );
	return false;
}

FString FWindowsPlatformProcess::GetApplicationName( uint32 ProcessId )
{
	FString Output;
	HANDLE ProcessHandle = ::OpenProcess(PROCESS_QUERY_INFORMATION, false, ProcessId);
	if (ProcessHandle != NULL)
	{
		const int32 ProcessNameBufferSize = 4096;
		TCHAR ProcessNameBuffer[ProcessNameBufferSize];
		
		int32 InOutSize = ProcessNameBufferSize;
		static_assert(sizeof(::DWORD) == sizeof(int32), "DWORD size doesn't match int32. Is it the future or the past?");

		if (QueryFullProcessImageName(ProcessHandle, 0, ProcessNameBuffer, (PDWORD)(&InOutSize)))
		{
			// TODO no null termination guarantee on GetProcessImageFileName?  it returns size as well, whereas QueryFullProcessImageName just returns non-zero on success
			Output = ProcessNameBuffer;
		}

		::CloseHandle(ProcessHandle);
	}

	return Output;
}

void FWindowsPlatformProcess::ReadFromPipes(FString* OutStrings[], HANDLE InPipes[], int32 PipeCount)
{
	for (int32 PipeIndex = 0; PipeIndex < PipeCount; ++PipeIndex)
	{
		if (InPipes[PipeIndex] && OutStrings[PipeIndex])
		{
			*OutStrings[PipeIndex] += ReadPipe(InPipes[PipeIndex]);
		}
	}
}

#include "Windows/AllowWindowsPlatformTypes.h"
/**
 * Executes a process, returning the return code, stdout, and stderr. This
 * call blocks until the process has returned.
 */
bool FWindowsPlatformProcess::ExecProcess(const TCHAR* URL, const TCHAR* Params, int32* OutReturnCode, FString* OutStdOut, FString* OutStdErr, const TCHAR* OptionalWorkingDirectory, bool bShouldEndWithParentProcess)
{
	if (bShouldEndWithParentProcess && !IsJobObjectSet)
	{
		GhJob = CreateJobObject(NULL, NULL);
		if (!GhJob)
		{
			UE_LOGF(LogWindows, Warning, "Failed to create Job Object");
		}
		else
		{
			JOBOBJECT_EXTENDED_LIMIT_INFORMATION LimitInformation;
			FPlatformMemory::Memzero(&LimitInformation, sizeof(LimitInformation));

			LimitInformation.BasicLimitInformation.LimitFlags = JOB_OBJECT_LIMIT_KILL_ON_JOB_CLOSE;
			if (0 == SetInformationJobObject(GhJob, JobObjectExtendedLimitInformation, &LimitInformation, sizeof(LimitInformation)))
			{
				UE_LOGF(LogWindows, Warning, "Could not SetInformationJobObject");
			}
		}
		IsJobObjectSet = true;
	}

	STARTUPINFOEX StartupInfoEx;
	ZeroMemory(&StartupInfoEx, sizeof(StartupInfoEx));
	StartupInfoEx.StartupInfo.cb = sizeof(StartupInfoEx);
	StartupInfoEx.StartupInfo.dwX = CW_USEDEFAULT;
	StartupInfoEx.StartupInfo.dwY = CW_USEDEFAULT;
	StartupInfoEx.StartupInfo.dwXSize = CW_USEDEFAULT;
	StartupInfoEx.StartupInfo.dwYSize = CW_USEDEFAULT;
	StartupInfoEx.StartupInfo.dwFlags = STARTF_USESHOWWINDOW;
	StartupInfoEx.StartupInfo.wShowWindow = SW_SHOWMINNOACTIVE;
	StartupInfoEx.StartupInfo.hStdInput = ::GetStdHandle(STD_INPUT_HANDLE);

	HANDLE hStdOutRead = NULL;
	HANDLE hStdErrRead = NULL;
	// Required to exist until DeleteProcThreadAttributeList is called
	HANDLE InheritHandles[2];
	TArray<uint8> AttributeList;

	if(OutStdOut != nullptr || OutStdErr != nullptr)
	{
		StartupInfoEx.StartupInfo.dwFlags |= STARTF_USESTDHANDLES;

		SECURITY_ATTRIBUTES Attr;
		ZeroMemory(&Attr, sizeof(Attr));
		Attr.nLength = sizeof(SECURITY_ATTRIBUTES);
		Attr.bInheritHandle = TRUE;

		verify(::CreatePipe(&hStdOutRead, &StartupInfoEx.StartupInfo.hStdOutput, &Attr, 0));
		verify(::CreatePipe(&hStdErrRead, &StartupInfoEx.StartupInfo.hStdError, &Attr, 0));

		SIZE_T BufferSize = 0;
		if(!InitializeProcThreadAttributeList(NULL, 1, 0, &BufferSize) && GetLastError() == ERROR_INSUFFICIENT_BUFFER)
		{
			AttributeList.SetNum(BufferSize);
			StartupInfoEx.lpAttributeList = (LPPROC_THREAD_ATTRIBUTE_LIST)AttributeList.GetData();
			verify(InitializeProcThreadAttributeList(StartupInfoEx.lpAttributeList, 1, 0, &BufferSize));
		}

		InheritHandles[0] = StartupInfoEx.StartupInfo.hStdOutput;
		InheritHandles[1] = StartupInfoEx.StartupInfo.hStdError;
		verify(UpdateProcThreadAttribute(StartupInfoEx.lpAttributeList, 0, PROC_THREAD_ATTRIBUTE_HANDLE_LIST, InheritHandles, sizeof(InheritHandles), NULL, NULL));
	}

	bool bSuccess = false;

	FString CommandLine;
	if (URL[0] != '\"') // Don't quote executable name if it's already quoted
	{
		CommandLine = FString::Printf(TEXT("\"%s\" %s"), URL, Params); 
	}
	else
	{
		CommandLine = FString::Printf(TEXT("%s %s"), URL, Params);
	}

	// CREATE_NEW_PROCESS_GROUP for consistency with Unix-like platforms
	// CREATE_NO_WINDOW because DETACHED_PROCESS can cause console windows to appear if the child spawns a process
	// https://ikriv.com/dev/cpp/ConsoleProxy/flags has the best explanation
	uint32 CreateFlags = NORMAL_PRIORITY_CLASS | CREATE_NEW_PROCESS_GROUP | CREATE_NO_WINDOW;
	// We only want to add the EXTENDED_STARTUPINFO_PRESENT flag if StartupInfoEx.lpAttributeList is actually setup.
	// If StartupInfoEx.lpAttributeList is NULL when the EXTENDED_STARTUPINFO_PRESENT flag is used, then CreateProcess causes an Access Violation crash on some Win32 configurations.
	// This is specific to when the call is redirected to APIHook_CreateProcessW in AcLayers.dll rather than the standard CreateProcessW implementation in kernel32.dll.
	if (StartupInfoEx.lpAttributeList != NULL)
	{
		CreateFlags |= EXTENDED_STARTUPINFO_PRESENT;
	}

	PROCESS_INFORMATION ProcInfo;
	if (CreateProcess(NULL, CommandLine.GetCharArray().GetData(), NULL, NULL, TRUE, CreateFlags, NULL, OptionalWorkingDirectory, &StartupInfoEx.StartupInfo, &ProcInfo))
	{
		if (bShouldEndWithParentProcess && GhJob)
		{
			int RetVal = AssignProcessToJobObject(GhJob, ProcInfo.hProcess);
			if (RetVal == 0)
			{
				UE_LOGF(LogWindows, Warning, "AssignProcessToObject failed.");
			}
		}
		if (hStdOutRead != NULL)
		{
			HANDLE ReadablePipes[2] = { hStdOutRead, hStdErrRead };
			FString* OutStrings[2] = { OutStdOut, OutStdErr };
			TArray<uint8> PipeBytes[2];

			auto ReadPipes = [&]()
			{
				for (int32 PipeIndex = 0; PipeIndex < 2; ++PipeIndex)
				{
					if (ReadablePipes[PipeIndex] && OutStrings[PipeIndex])
					{
						TArray<uint8> BinaryData;
						ReadPipeToArray(ReadablePipes[PipeIndex], BinaryData);
						PipeBytes[PipeIndex].Append(BinaryData);
					}
				}
			};

			FProcHandle ProcHandle(ProcInfo.hProcess);
			do
			{
				ReadPipes();
				FPlatformProcess::Sleep(0);
			} while (IsProcRunning(ProcHandle));
			ReadPipes();

			// Convert only after all bytes are available to prevent string corruption
			for (int32 PipeIndex = 0; PipeIndex < 2; ++PipeIndex)
			{
				if (OutStrings[PipeIndex] && PipeBytes[PipeIndex].Num() > 0)
				{
					PipeBytes[PipeIndex].Add('\0');
					*OutStrings[PipeIndex] = FUTF8ToTCHAR((const ANSICHAR*)PipeBytes[PipeIndex].GetData()).Get();
				}
			}
		}
		else
		{
			::WaitForSingleObject(ProcInfo.hProcess, INFINITE);
		}
		if (OutReturnCode)
		{
			verify(::GetExitCodeProcess(ProcInfo.hProcess, (DWORD*)OutReturnCode));
		}
		::CloseHandle(ProcInfo.hProcess);
		::CloseHandle(ProcInfo.hThread);
		bSuccess = true;
	}
	else
	{
		DWORD ErrorCode = GetLastError();

		// if CreateProcess failed, we should return a useful error code, which GetLastError will have
		if (OutReturnCode)
		{
			*OutReturnCode = ErrorCode;
		}

		TCHAR ErrorMessage[512];
		FWindowsPlatformMisc::GetSystemErrorMessage(ErrorMessage, 512, ErrorCode);

		UE_LOGF(LogWindows, Warning, "CreateProc failed: %ls (0x%08x)", ErrorMessage, ErrorCode);
		if (ErrorCode == ERROR_NOT_ENOUGH_MEMORY || ErrorCode == ERROR_OUTOFMEMORY)
		{
			// These errors are common enough that we want some available memory information
			FPlatformMemoryStats Stats = FPlatformMemory::GetStats();
			UE_LOGF(LogWindows, Warning, "Mem used: %.2f MB, OS Free %.2f MB", Stats.UsedPhysical / 1048576.0f, Stats.AvailablePhysical / 1048576.0f);
		}
		UE_LOGF(LogWindows, Warning, "URL: %ls %ls", URL, Params);
	}

	if(StartupInfoEx.StartupInfo.hStdOutput != NULL)
	{
		CloseHandle(StartupInfoEx.StartupInfo.hStdOutput);
	}
	if(StartupInfoEx.StartupInfo.hStdError != NULL)
	{
		CloseHandle(StartupInfoEx.StartupInfo.hStdError);
	}
	if(hStdOutRead != NULL)
	{
		CloseHandle(hStdOutRead);
	}
	if(hStdErrRead != NULL)
	{
		CloseHandle(hStdErrRead);
	}

	if(StartupInfoEx.lpAttributeList != NULL)
	{
		DeleteProcThreadAttributeList(StartupInfoEx.lpAttributeList);
	}

	return bSuccess;
}
#include "Windows/HideWindowsPlatformTypes.h"

bool FWindowsPlatformProcess::ExecElevatedProcess(const TCHAR* URL, const TCHAR* Params, int32* OutReturnCode)
{
	SHELLEXECUTEINFO ShellExecuteInfo;
	ZeroMemory(&ShellExecuteInfo, sizeof(ShellExecuteInfo));
	ShellExecuteInfo.cbSize = sizeof(ShellExecuteInfo);
	ShellExecuteInfo.fMask = SEE_MASK_UNICODE | SEE_MASK_NOCLOSEPROCESS;
	ShellExecuteInfo.lpFile = URL;
	ShellExecuteInfo.lpVerb = TEXT("runas");
	ShellExecuteInfo.nShow = SW_SHOW;
	ShellExecuteInfo.lpParameters = Params;

	bool bSuccess = false;
	if (ShellExecuteEx(&ShellExecuteInfo))
	{
		::WaitForSingleObject(ShellExecuteInfo.hProcess, INFINITE);
		if (OutReturnCode != NULL)
		{
			verify(::GetExitCodeProcess(ShellExecuteInfo.hProcess, (::DWORD*)OutReturnCode));
		}
		verify(::CloseHandle(ShellExecuteInfo.hProcess));
		bSuccess = true;
	}
	return bSuccess;
}

FProcHandle FWindowsPlatformProcess::CreateElevatedProcess(const TCHAR* URL, const TCHAR* Params)
{
	SHELLEXECUTEINFO ShellExecuteInfo;
	ZeroMemory(&ShellExecuteInfo, sizeof(ShellExecuteInfo));
	ShellExecuteInfo.cbSize = sizeof(ShellExecuteInfo);
	ShellExecuteInfo.fMask = SEE_MASK_UNICODE | SEE_MASK_NOCLOSEPROCESS;
	ShellExecuteInfo.lpFile = URL;
	ShellExecuteInfo.lpVerb = TEXT("runas");
	ShellExecuteInfo.nShow = SW_SHOW;
	ShellExecuteInfo.lpParameters = Params;

	if (ShellExecuteEx(&ShellExecuteInfo))
	{
		return FProcHandle(ShellExecuteInfo.hProcess);
	}

	return FProcHandle{};
}

static TCHAR GBaseDirectory[512] = TEXT("");

const TCHAR* FWindowsPlatformProcess::BaseDir()
{
	if( !GBaseDirectory[0] )
	{
#if WINDOWS_ALLOW_BASEDIR_OVERRIDE
		// Normally the BaseDir is determined from the path of the running process module,
		// but for debugging, particularly client or server, it can be useful to point the
		// code at an existing cooked directory. If using -BaseFromWorkingDir set the
		// workingdir in your debugger to the <path>/Project/Binaries/Win64 of your cooked
		// data
		// Too early to use the FCommand line interface
		FString BaseArg;
		FParse::Value(::GetCommandLineW(), TEXT("-basedir="), BaseArg);

		if (BaseArg.Len())
		{
			BaseArg = BaseArg.Replace(TEXT("\\"), TEXT("/"));
			if (!BaseArg.EndsWith(TEXT("/")))
			{
				BaseArg += TEXT('/');
			}
			FCString::Strcpy(GBaseDirectory, *BaseArg);
		}
		else if (FCString::Stristr(::GetCommandLineW(), TEXT("-BaseFromWorkingDir")))
		{
			::GetCurrentDirectory(512, GBaseDirectory);

			FString TempResult(GBaseDirectory);
			TempResult = TempResult.Replace(TEXT("\\"), TEXT("/"));
			if (!TempResult.EndsWith(TEXT("/")))
			{
				TempResult += TEXT('/');
			}
			FCString::Strcpy(GBaseDirectory, *TempResult);
		}
		else
#endif // WINDOWS_ALLOW_BASEDIR_OVERRIDE
		{
			// Get the directory containing the current module if possible, or use the directory containing the executable if not
			HMODULE hCurrentModule;
			if (GetModuleHandleEx(GET_MODULE_HANDLE_EX_FLAG_FROM_ADDRESS | GET_MODULE_HANDLE_EX_FLAG_UNCHANGED_REFCOUNT, (LPCWSTR)&BaseDir, &hCurrentModule) == 0)
			{
				hCurrentModule = hInstance;
			}
			GetModuleFileName(hCurrentModule, GBaseDirectory, UE_ARRAY_COUNT(GBaseDirectory));
			FString TempResult(GBaseDirectory);
			TempResult = TempResult.Replace(TEXT("\\"), TEXT("/"));

			FCString::Strcpy(GBaseDirectory, *TempResult);
			int32 StringLength = FCString::Strlen(GBaseDirectory);
			if (StringLength > 0)
			{
				--StringLength;
				for (; StringLength > 0; StringLength--)
				{
					if (GBaseDirectory[StringLength - 1] == TEXT('/') || GBaseDirectory[StringLength - 1] == TEXT('\\'))
					{
						break;
					}
				}
			}
			GBaseDirectory[StringLength] = TCHAR('\0');

			FString CollapseResult(GBaseDirectory);
#ifdef UE_RELATIVE_BASE_DIR
			// this may have been defined at compile time because we are in Restricted, but then we have been staged as a program, and then remapped out of Restricted
			// so if we are already in a Binaries/Win64 directory
			if (IFileManager::Get().DirectoryExists(*FPaths::Combine(CollapseResult, UE_RELATIVE_BASE_DIR)))
			{
				CollapseResult = FPaths::Combine(CollapseResult, UE_RELATIVE_BASE_DIR);
			}
#endif
			FPaths::CollapseRelativeDirectories(CollapseResult);
			FCString::Strcpy(GBaseDirectory, *CollapseResult);
		}
	}
	return GBaseDirectory;
}

#if WITH_LOW_LEVEL_TESTS
void FWindowsPlatformProcess::SetBaseDirForTesting(const TCHAR* InBaseDir)
{
	FCString::Strncpy(GBaseDirectory, InBaseDir, UE_ARRAY_COUNT(GBaseDirectory));
}
#endif

const TCHAR* FWindowsPlatformProcess::UserDir()
{
	static FString WindowsUserDir;
	if( !WindowsUserDir.Len() )
	{
		TCHAR* UserPath;

		// get the My Documents directory
		HRESULT Ret = SHGetKnownFolderPath(FOLDERID_Documents, 0, NULL, &UserPath);
		if (SUCCEEDED(Ret))
		{
			// make the base user dir path
			WindowsUserDir = FString(UserPath).Replace(TEXT("\\"), TEXT("/")) + TEXT("/");
			CoTaskMemFree(UserPath);
		}
	}
	return *WindowsUserDir;
}

const TCHAR* FWindowsPlatformProcess::UserTempDir()
{
	static FString WindowsUserTempDir;
	if( !WindowsUserTempDir.Len() )
	{
		// Windows temp dir functions don't understand integrity levels so we have to build our own path to AppData\LocalLow\Temp.
		if (ShouldExpectLowIntegrityLevel())
		{
			TCHAR* UserPath;
			HRESULT Ret = SHGetKnownFolderPath(FOLDERID_LocalAppDataLow, 0, NULL, &UserPath);
			if (SUCCEEDED(Ret))
			{
				WindowsUserTempDir = FString(UserPath).Replace(TEXT("\\"), TEXT("/")) + TEXT("/Temp/");
				CoTaskMemFree(UserPath);
			}
		}
		else
		{
			TCHAR TempPath[MAX_PATH];
			ZeroMemory(TempPath, sizeof(TCHAR) * MAX_PATH);

			::GetTempPath(MAX_PATH, TempPath);

			// Always expand the temp path in case windows returns short directory names.
			TCHAR FullTempPath[MAX_PATH];
			ZeroMemory(FullTempPath, sizeof(TCHAR) * MAX_PATH);
			::GetLongPathName(TempPath, FullTempPath, MAX_PATH);

			WindowsUserTempDir = FString(FullTempPath).Replace(TEXT("\\"), TEXT("/"));
		}
	}
	return *WindowsUserTempDir;
}

const TCHAR* FWindowsPlatformProcess::UserSettingsDir()
{
	static FString WindowsUserSettingsDir;
	if (!WindowsUserSettingsDir.Len())
	{
		TCHAR* UserPath;

		// get the local or locallow AppData directory depending on integrity configuration
		HRESULT Ret = SHGetKnownFolderPath(ShouldExpectLowIntegrityLevel() ? FOLDERID_LocalAppDataLow : FOLDERID_LocalAppData, 0, NULL, &UserPath);
		if (SUCCEEDED(Ret))
		{
			// make the base user dir path
			WindowsUserSettingsDir = FString(UserPath).Replace(TEXT("\\"), TEXT("/")) + TEXT("/");
			CoTaskMemFree(UserPath);
		}
	}
	return *WindowsUserSettingsDir;
}

const TCHAR *FWindowsPlatformProcess::UserHomeDir()
{
	static FString WindowsUserHomeDir = [&]() -> FString 
	{
		if (TCHAR* UserPath; SUCCEEDED(SHGetKnownFolderPath(FOLDERID_Profile, 0, nullptr, &UserPath)))
		{
			FString Directory = UserPath;
			Directory.ReplaceInline(TEXT("\\"), TEXT("/"));
			Directory.Append(TEXT("/"));
			Directory.Shrink();
			CoTaskMemFree(UserPath);
			return Directory;
		}

		return {};
	}();
	return *WindowsUserHomeDir;
}

const TCHAR* FWindowsPlatformProcess::UserSettingsDirMediumIntegrity()
{
	static FString WindowsUserSettingsDir;
	if (!WindowsUserSettingsDir.Len())
	{
		TCHAR* UserPath;

		// get the local AppData directory
		HRESULT Ret = SHGetKnownFolderPath(FOLDERID_LocalAppData, 0, NULL, &UserPath);
		if (SUCCEEDED(Ret))
		{
			// make the base user dir path
			WindowsUserSettingsDir = FString(UserPath).Replace(TEXT("\\"), TEXT("/")) + TEXT("/");
			CoTaskMemFree(UserPath);
		}
	}
	return *WindowsUserSettingsDir;
}

const TCHAR* FWindowsPlatformProcess::ApplicationSettingsDir()
{
	static FString WindowsApplicationSettingsDir;
	if( !WindowsApplicationSettingsDir.Len() )
	{
		TCHAR* ApplictionSettingsPath;

		// get the local AppData directory
		HRESULT Ret = SHGetKnownFolderPath(FOLDERID_ProgramData, 0, NULL, &ApplictionSettingsPath);
		if (SUCCEEDED(Ret))
		{
			// make the base user dir path
			WindowsApplicationSettingsDir = FString(ApplictionSettingsPath).Replace(TEXT("\\"), TEXT("/")) + TEXT("/Epic/");
			CoTaskMemFree(ApplictionSettingsPath);
		}
	}
	return *WindowsApplicationSettingsDir;
}

FString FWindowsPlatformProcess::GetApplicationSettingsDir(const ApplicationSettingsContext& Settings)
{
	FString WindowsApplicationSettingsDir;
	TCHAR* ApplicationSettingsPath;
	switch (Settings.Location)
	{
		case ApplicationSettingsContext::Context::ApplicationSpecific:
		{
			const HRESULT Ret = SHGetKnownFolderPath(FOLDERID_ProgramData, 0, NULL, &ApplicationSettingsPath);
			if (SUCCEEDED(Ret))
			{
				WindowsApplicationSettingsDir = FString(ApplicationSettingsPath);
				break;
			}
			else
			{
				return "";
			}
		}
		case ApplicationSettingsContext::Context::LocalUser:
		{
			const HRESULT Ret = SHGetKnownFolderPath(ShouldExpectLowIntegrityLevel() ? FOLDERID_LocalAppDataLow : FOLDERID_LocalAppData, 0, NULL, &ApplicationSettingsPath);
			if (SUCCEEDED(Ret))
			{
				WindowsApplicationSettingsDir = FString(ApplicationSettingsPath);
				break;
			}
			else
			{
				return "";
			}
		}
		case ApplicationSettingsContext::Context::RoamingUser:
		{
			const HRESULT Ret = SHGetKnownFolderPath(FOLDERID_RoamingAppData, 0, NULL, &ApplicationSettingsPath);
			if (SUCCEEDED(Ret))
			{
				WindowsApplicationSettingsDir = FString(ApplicationSettingsPath);
				break;
			}
			else
			{
				return "";
			}
		}
		default:
			checkf(false, TEXT("Attempting to call `ApplicationSettingsDir` with an invalid context!"));
			return "";
	}
	WindowsApplicationSettingsDir = WindowsApplicationSettingsDir.Replace(TEXT("\\"), TEXT("/")) + TEXT("/");
	CoTaskMemFree(ApplicationSettingsPath);
	if (Settings.bIsEpic)
	{
		WindowsApplicationSettingsDir += TEXT("Epic/");
	}
	return WindowsApplicationSettingsDir;
}

const TCHAR* FWindowsPlatformProcess::WindowsSystemDir()
{
	static FString WindowsSystemDir;
	if( !WindowsSystemDir.Len() )
	{
		TCHAR* SystemPath;

		HRESULT Ret = SHGetKnownFolderPath(FOLDERID_System, 0, nullptr, &SystemPath);
		if (SUCCEEDED(Ret))
		{
			WindowsSystemDir = FString(SystemPath).Replace(TEXT("\\"), TEXT("/"));
			CoTaskMemFree(SystemPath);
		}
	}
	return *WindowsSystemDir;

}

const TCHAR* FWindowsPlatformProcess::ComputerName()
{
	static TCHAR Result[256] = {};
	if( !Result[0] )
	{
		uint32 Size=UE_ARRAY_COUNT(Result);
		GetComputerName( Result, (::DWORD*)&Size );
	}
	return Result;
}

const TCHAR* FWindowsPlatformProcess::UserName(bool bOnlyAlphaNumeric/* = true*/)
{
	static TCHAR Result[256] = {};
	static TCHAR ResultAlpha[256] = {};
	if( bOnlyAlphaNumeric )
	{
		if( !ResultAlpha[0] )
		{
			uint32 Size=UE_ARRAY_COUNT(ResultAlpha);
			GetUserName( ResultAlpha, (::DWORD*)&Size );
			TCHAR *c, *d;
			for( c=ResultAlpha, d=ResultAlpha; *c!=0; c++ )
				if( FChar::IsAlnum(*c) )
					*d++ = *c;
			*d++ = TCHAR('\0');
		}
		return ResultAlpha;
	}
	else
	{
		if( !Result[0] )
		{
			uint32 Size=UE_ARRAY_COUNT(Result);
			GetUserName( Result, (::DWORD*)&Size );
		}
		return Result;
	}
}

void FWindowsPlatformProcess::SetCurrentWorkingDirectoryToBaseDir()
{
#if defined(DISABLE_CWD_CHANGES) && DISABLE_CWD_CHANGES != 0
	checkf(false, TEXT("Attempting to call 'SetCurrentWorkingDirectoryToBaseDir' while DISABLE_CWD_CHANGES is set!"));
#else
	FPlatformMisc::CacheLaunchDir();

	// Ideally we would log the following errors but this is most likely to fail right at the start of the 
	// program and any call to UE_LOG at this point will not actually result in anything being written to disk.
#if DO_CHECK
	TCHAR SystemError[1024];
#endif //DO_CHECK
	
	verifyf(::SetCurrentDirectoryW(BaseDir()),	TEXT("Failed to set the working directory to '%s' (%s)"), 
												BaseDir(), 
												FWindowsPlatformMisc::GetSystemErrorMessage(SystemError, UE_ARRAY_COUNT(SystemError), 0));
#endif //DISABLE_CWD_CHANGES
}

/** Get the current working directory (only really makes sense on desktop platforms) */
FString FWindowsPlatformProcess::GetCurrentWorkingDirectory()
{
	// Allocate the data for the string. Loop in case the variable happens to change while running, or the buffer isn't large enough.
	FString Buffer;
	for (uint32 Length = 128;;)
	{
		TArray<TCHAR, FString::AllocatorType>& CharArray = Buffer.GetCharArray();
		CharArray.SetNumUninitialized(Length);

		Length = ::GetCurrentDirectoryW(CharArray.Num(), CharArray.GetData());
		if (Length == 0)
		{
			Buffer.Reset();
			break;
		}
		if (Length < (uint32)CharArray.Num())
		{
			CharArray.SetNum(Length + 1);
			break;
		}
	}
	return Buffer;
}

const FString FWindowsPlatformProcess::ShaderWorkingDir()
{
	return (FString(FPlatformProcess::UserTempDir()) / TEXT("UnrealShaderWorkingDir/"));
}


const TCHAR* FWindowsPlatformProcess::ExecutablePath()
{
	static TCHAR Result[512] = {};
	if( !Result[0] )
	{
		if ( !GetModuleFileName( hInstance, Result, UE_ARRAY_COUNT(Result) ) )
		{
			Result[0] = TCHAR('\0');
		}
	}
	return Result;
}

const TCHAR* FWindowsPlatformProcess::ExecutableName(bool bRemoveExtension)
{
	static TCHAR Result[512] = {};
	static TCHAR ResultWithExt[512] = {};
	if( !Result[0] )
	{
		// Get complete path for the executable
		if ( GetModuleFileName( hInstance, Result, UE_ARRAY_COUNT(Result) ) != 0 )
		{
			// Remove all of the path information by finding the base filename
			FString FileName = Result;
			FString FileNameWithExt = Result;
			FCString::Strncpy( Result, *( FPaths::GetBaseFilename(FileName) ), UE_ARRAY_COUNT(Result) );
			FCString::Strncpy( ResultWithExt, *( FPaths::GetCleanFilename(FileNameWithExt) ), UE_ARRAY_COUNT(ResultWithExt) );
		}
		// If the call failed, zero out the memory to be safe
		else
		{
			FMemory::Memzero( Result, sizeof( Result ) );
			FMemory::Memzero( ResultWithExt, sizeof( ResultWithExt ) );
		}
	}

	return (bRemoveExtension ? Result : ResultWithExt);
}

const TCHAR* FWindowsPlatformProcess::GetModuleExtension()
{
	return TEXT("dll");
}

const TCHAR* FWindowsPlatformProcess::GetBinariesSubdirectory()
{
	return TEXT("Win64");
}

const FString FWindowsPlatformProcess::GetModulesDirectory()
{
	static TCHAR Result[MAX_PATH];
	if(Result[0] == 0)
	{
		// Get the handle to the current module
		HMODULE hCurrentModule;
		if (GetModuleHandleEx(GET_MODULE_HANDLE_EX_FLAG_FROM_ADDRESS | GET_MODULE_HANDLE_EX_FLAG_UNCHANGED_REFCOUNT, (LPCWSTR)&GetModulesDirectory, &hCurrentModule) == 0)
		{
			hCurrentModule = hInstance;
		}

		// Get the directory for it
		GetModuleFileName(hCurrentModule, Result, UE_ARRAY_COUNT(Result));
		*FCString::Strrchr(Result, '\\') = 0;

		// Normalize the resulting path
		FString Buffer = Result;
		FPaths::MakeStandardFilename(Buffer);
		FCString::Strcpy(Result, *Buffer);
	}
	return Result;
}

bool FWindowsPlatformProcess::LaunchFileInDefaultExternalApplication( const TCHAR* FileName, const TCHAR* Parms /*= NULL*/, ELaunchVerb::Type Verb /*= ELaunchVerb::Open*/, bool bPromptToOpenOnFailure /*= true */ )
{
	const TCHAR* VerbString = Verb == ELaunchVerb::Edit ? TEXT("edit") : TEXT("open");

	// First attempt to open the file in its default application
	UE_LOGF(LogWindows, Log,  "LaunchFileInExternalEditor %ls %ls", FileName, Parms ? Parms : TEXT("") );
	HINSTANCE Code = ::ShellExecuteW( NULL, VerbString, FileName, Parms ? Parms : TEXT(""), TEXT(""), SW_SHOWNORMAL );
	
	UE_LOG(LogWindows, Log,  TEXT("Launch application code for %s %s: %" PTRINT_FMT), FileName, Parms ? Parms : TEXT(""), (PTRINT)Code );

	// Fallback to a true windows-defined default for the asset type
	if ((PTRINT)Code == SE_ERR_NOASSOC || (PTRINT)Code == SE_ERR_ASSOCINCOMPLETE)
	{
		Code = ::ShellExecuteW(NULL, NULL, FileName, NULL, NULL, SW_SHOW);
	}

	// If opening the file in the default application failed, check to see if it's because the file's extension does not have
	// a default application associated with it. If so, prompt the user with the Windows "Open With..." dialog to allow them to specify
	// an application to use.
	if (bPromptToOpenOnFailure && ((PTRINT)Code == SE_ERR_NOASSOC || (PTRINT)Code == SE_ERR_ASSOCINCOMPLETE))
	{
		Code = ::ShellExecuteW( NULL, VerbString, TEXT("RUNDLL32.EXE"), *FString::Printf( TEXT("shell32.dll,OpenAs_RunDLL %s"), FileName ), TEXT(""), SW_SHOWNORMAL );
	}

	// If code is > 32, it's a valid handle, return true. Otherwise opening the file failed, return false.
	return ((PTRINT)Code > 32);
}


void FWindowsPlatformProcess::ExploreFolder( const TCHAR* FilePath )
{
	if (IPlatformFile::GetPlatformPhysical().DirectoryExists( FilePath ))
	{
		// Explore the folder
		::ShellExecuteW( NULL, TEXT("explore"), FilePath, NULL, NULL, SW_SHOWNORMAL );
	}
	else
	{
		// Explore the file
		FString NativeFilePath = FString(FilePath).Replace(TEXT("/"), TEXT("\\"));
		FString Parameters = FString::Printf( TEXT("/select,%s"), *NativeFilePath);
		::ShellExecuteW( NULL, TEXT("open"), TEXT("explorer.exe"), *Parameters, NULL, SW_SHOWNORMAL );
	}
}

/**
 * Resolves UNC path to local (full) path if possible.
 *
 * @param	InUNCPath		UNC path to resolve
 *
 * @param	OutPath		Resolved local path
 *
 * @return true if the path was resolved, false otherwise
 */
bool FWindowsPlatformProcess::ResolveNetworkPath( FString InUNCPath, FString& OutPath )
{
	// Get local machine name first and check if this UNC path points to local share
	// (if it's not UNC path it will also fail this check)
	uint32 ComputerNameSize = MAX_COMPUTERNAME_LENGTH;
	TCHAR ComputerName[MAX_COMPUTERNAME_LENGTH + 3] = { TEXT('\\'), TEXT('\\'), TEXT('\0'), };

	if ( GetComputerName( ComputerName + 2, (::DWORD*)&ComputerNameSize ) )
	{
		// Check if the filename is pointing to local shared folder
		if ( InUNCPath.StartsWith( ComputerName ) )
		{
			// Get the share name (it's the first folder after the computer name)
			int32 ComputerNameLen = FCString::Strlen( ComputerName );
			int32 ShareNameLen = InUNCPath.Find( TEXT( "\\" ), ESearchCase::CaseSensitive, ESearchDir::FromStart, ComputerNameLen + 1 ) - ComputerNameLen - 1;
			FString ShareName = InUNCPath.Mid( ComputerNameLen + 1, ShareNameLen );

			// NetShareGetInfo doesn't accept const TCHAR* as the share name so copy to temp array
			SHARE_INFO_2* BufPtr = NULL;
			::NET_API_STATUS res;

			// Call the NetShareGetInfo function, specifying level 2
			if ( ( res = NetShareGetInfo( NULL, ShareName.GetCharArray().GetData(), 2, (LPBYTE*)&BufPtr ) ) == ERROR_SUCCESS )
			{
				// Construct the local path
				OutPath = FString( BufPtr->shi2_path ) + InUNCPath.Mid( ComputerNameLen + 1 + ShareNameLen );

				// Free the buffer allocated by NetShareGetInfo
				NetApiBufferFree(BufPtr);
				
				return true;
			}
		}
	}

	// InUNCPath is not an UNC path or it's not pointing to local folder or something went wrong in NetShareGetInfo (insufficient privileges?)
	return false;
}

void FWindowsPlatformProcess::Sleep( float Seconds )
{
	SCOPE_CYCLE_COUNTER( STAT_Sleep );
	UE::Stats::FThreadIdleStats::FScopeIdle Scope;
	SleepNoStats( Seconds );
}

void FWindowsPlatformProcess::SleepNoStats(float Seconds)
{
	uint32 Milliseconds = (uint32)(Seconds * 1000.0);
	if (Milliseconds == 0)
	{
		::SwitchToThread();
	}
	else
	{
		::Sleep(Milliseconds);
	}
}

void FWindowsPlatformProcess::SleepInfinite()
{
	while (true)
	{
		::Sleep(INFINITE);
	}
}

void FWindowsPlatformProcess::YieldThread()
{
	::SwitchToThread();
}

#include "WindowsEvent.h"

FEvent* FWindowsPlatformProcess::CreateSynchEvent(bool bIsManualReset)
{
	// While windows does not support forking we can still simulate the forking codeflow and test the singlethread to multithread switch on Win targets
	const bool bIsMultithread = FPlatformProcess::SupportsMultithreading() || FForkProcessHelper::SupportsMultithreadingPostFork();

	// Allocate the new object
	FEvent* Event = NULL;	
	if (bIsMultithread)
	{
		Event = new FEventWin();
	}
	else
	{
		// Fake event object.
		Event = new FSingleThreadEvent();
	}
	// If the internal create fails, delete the instance and return NULL
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (!Event->Create(bIsManualReset))
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		delete Event;
		Event = NULL;
	}
	return Event;
}

#include "Windows/AllowWindowsPlatformTypes.h"

bool FEventWin::Wait(uint32 WaitTime, const bool bIgnoreThreadIdleStats /*= false*/)
{
	WaitForStats();

	SCOPE_CYCLE_COUNTER( STAT_EventWait );
	CSV_SCOPED_WAIT(WaitTime);
	check(Event);

	LowLevelTasks::FOversubscriptionScope _(WaitTime != 0); // Let the scheduler know one of its thread might be waiting.
	UE::Stats::FThreadIdleStats::FScopeIdle Scope( bIgnoreThreadIdleStats );
	return (WaitForSingleObject( Event, WaitTime ) == WAIT_OBJECT_0);
}

void FEventWin::Trigger()
{
	TriggerForStats();
	check( Event );
	SetEvent( Event );
}

void FEventWin::Reset()
{
	ResetForStats();
	check( Event );
	ResetEvent( Event );
}

#include "Windows/HideWindowsPlatformTypes.h"

#include "WindowsRunnableThread.h"

FRunnableThread* FWindowsPlatformProcess::CreateRunnableThread()
{
	return new FRunnableThreadWin();
}

void FWindowsPlatformProcess::ClosePipe( void* ReadPipe, void* WritePipe )
{
	if ( ReadPipe != NULL && ReadPipe != INVALID_HANDLE_VALUE )
	{
		::CloseHandle(ReadPipe);
	}
	if ( WritePipe != NULL && WritePipe != INVALID_HANDLE_VALUE )
	{
		::CloseHandle(WritePipe);
	}
}

bool FWindowsPlatformProcess::CreatePipe( void*& ReadPipe, void*& WritePipe, bool bWritePipeLocal )
{
	SECURITY_ATTRIBUTES Attr = { sizeof(SECURITY_ATTRIBUTES), NULL, true };
	
	if (!::CreatePipe(&ReadPipe, &WritePipe, &Attr, 0))
	{
		return false;
	}

	if (!::SetHandleInformation(bWritePipeLocal ? WritePipe : ReadPipe, HANDLE_FLAG_INHERIT, 0))
	{
		return false;
	}

	return true;
}

FString FWindowsPlatformProcess::ReadPipe(void* ReadPipe)
{
	// Note: String becomes corrupted when more than one byte per character and all bytes are not available
	::DWORD BytesAvailable;
	if (::PeekNamedPipe(ReadPipe, nullptr, 0, nullptr, &BytesAvailable, nullptr) && BytesAvailable)
	{
		TUtf8StringBuilder<256> Buffer;
		Buffer.AddUninitialized(int32(BytesAvailable));
		::DWORD BytesRead;
		if (::ReadFile(ReadPipe, Buffer.GetData(), BytesAvailable, &BytesRead, nullptr))
		{
			return FString{Buffer.ToView().Left(int32(BytesRead))};
		}
	}

	return {};
}

bool FWindowsPlatformProcess::ReadPipeToArray(void* ReadPipe, TArray<uint8> & Output)
{
	uint32 BytesAvailable = 0;
	if (::PeekNamedPipe(ReadPipe, NULL, 0, NULL, (::DWORD*)&BytesAvailable, NULL) && (BytesAvailable > 0))
	{
		Output.SetNumUninitialized(BytesAvailable);
		uint32 BytesRead = 0;
		if (::ReadFile(ReadPipe, Output.GetData(), BytesAvailable, (::DWORD*)&BytesRead, NULL))
		{
			if (BytesRead < BytesAvailable)
			{
				Output.SetNum(BytesRead);
			}

			return true;
		}
		else
		{
			Output.Empty();
		}
	}

	return false;
}

bool FWindowsPlatformProcess::WritePipe(void* WritePipe, const FString& Message, FString* OutWritten)
{
	// If there is not a message or WritePipe is null
	int32 MessageLen = Message.Len();
	if (MessageLen == 0 || WritePipe == nullptr)
	{
		return false;
	}

	// Convert input to UTF8CHAR
	const TCHAR* MessagePtr = *Message;
	int32 BytesAvailable = FPlatformString::ConvertedLength<UTF8CHAR>(MessagePtr, MessageLen);
	UTF8CHAR* Buffer = new UTF8CHAR[BytesAvailable + 2];
	*FPlatformString::Convert(Buffer, BytesAvailable, MessagePtr, MessageLen) = (UTF8CHAR)'\n';

	// Write to pipe
	uint32 BytesWritten = 0;
	bool bIsWritten = !!WriteFile(WritePipe, Buffer, BytesAvailable + 1, (::DWORD*)&BytesWritten, nullptr);

	// Get written message
	if (OutWritten)
	{
		*OutWritten = StringCast<TCHAR>(Buffer, BytesWritten).Get();
	}

	delete[] Buffer;
	return bIsWritten;
}

bool FWindowsPlatformProcess::WritePipe(void* WritePipe, const uint8* Data, const int32 DataLength, int32* OutDataLength)
{
	// if there is not a message or WritePipe is null
	if ((DataLength == 0) || (WritePipe == nullptr))
	{
		return false;
	}

	// write to pipe
	uint32 BytesWritten = 0;
	bool bIsWritten = !!WriteFile(WritePipe, Data, DataLength, (::DWORD*)&BytesWritten, nullptr);

	// Get written Data Length
	if (OutDataLength)
	{
		*OutDataLength = (int32)BytesWritten;
	}

	return bIsWritten;
}

#include "Windows/AllowWindowsPlatformTypes.h"

FWindowsPlatformProcess::FWindowsSemaphore::FWindowsSemaphore(const FString & InName, HANDLE InSemaphore)
	: FWindowsSemaphore(*InName, InSemaphore)
{
}

FWindowsPlatformProcess::FWindowsSemaphore::FWindowsSemaphore(const TCHAR* InName, Windows::HANDLE InSemaphore)
	: FSemaphore(InName)
	, Semaphore(InSemaphore)
{

}

FWindowsPlatformProcess::FWindowsSemaphore::~FWindowsSemaphore()
{
	// actual cleanup should be done in DeleteInterprocessSynchObject() since it can return errors
}

void FWindowsPlatformProcess::FWindowsSemaphore::Lock()
{
	check(Semaphore);
	DWORD WaitResult = WaitForSingleObject(Semaphore, INFINITE);
	if (WaitResult != WAIT_OBJECT_0)
	{
		DWORD ErrNo = GetLastError();
		UE_LOGF(LogHAL, Warning, "WaitForSingleObject(,INFINITE) for semaphore '%ls' failed with return code 0x%08x and LastError = %d",
			GetName(),
			WaitResult,
			ErrNo);
	}
}

bool FWindowsPlatformProcess::FWindowsSemaphore::TryLock(uint64 NanosecondsToWait)
{
	check(Semaphore);
	DWORD MillisecondsToWait = NanosecondsToWait / 1000000ULL;
	DWORD WaitResult = WaitForSingleObject(Semaphore, MillisecondsToWait);
	if (WaitResult != WAIT_OBJECT_0 && WaitResult != WAIT_TIMEOUT)	// timeout is not a warning
	{
		DWORD ErrNo = GetLastError();
		UE_LOGF(LogHAL, Warning, "WaitForSingleObject(,INFINITE) for semaphore '%ls' failed with return code 0x%08x and LastError = %d",
			GetName(),
			WaitResult,
			ErrNo);
	}

	return WaitResult == WAIT_OBJECT_0;
}

void FWindowsPlatformProcess::FWindowsSemaphore::Unlock()
{
	check(Semaphore);
	if (!ReleaseSemaphore(Semaphore, 1, NULL))
	{
		DWORD ErrNo = GetLastError();
		UE_LOGF(LogHAL, Warning, "ReleaseSemaphore(,ReleaseCount=1,) for semaphore '%ls' failed with LastError = %d",
			GetName(),
			ErrNo);
	}
}

FWindowsPlatformProcess::FSemaphore* FWindowsPlatformProcess::NewInterprocessSynchObject(const FString & Name, bool bCreate, uint32 MaxLocks)
{
	return NewInterprocessSynchObject(*Name, bCreate, MaxLocks);
}

FWindowsPlatformProcess::FSemaphore* FWindowsPlatformProcess::NewInterprocessSynchObject(const TCHAR* Name, bool bCreate, uint32 MaxLocks)
{
	HANDLE Semaphore = NULL;
	
	if (bCreate)
	{
		Semaphore = CreateSemaphore(NULL, MaxLocks, MaxLocks, Name);
		if (NULL == Semaphore)
		{
			DWORD ErrNo = GetLastError();
			UE_LOGF(LogHAL, Warning, "CreateSemaphore(Attrs=NULL, InitialValue=%d, MaxValue=%d, Name='%ls') failed with LastError = %d",
				MaxLocks, MaxLocks,
				Name,
				ErrNo);
			return NULL;
		}
	}
	else
	{
		DWORD AccessRights = SYNCHRONIZE | SEMAPHORE_MODIFY_STATE;
		Semaphore = OpenSemaphore(AccessRights, false, Name);
		if (NULL == Semaphore)
		{
			const DWORD ErrNo = GetLastError();

			// ERROR_FILE_NOT_FOUND is expected when checking if the semaphore exists but it doesn't,
			// so we log the failure to open using "Verbose". Other errors will be logged using "Warning" verbosity.

			if (ErrNo == ERROR_FILE_NOT_FOUND)
			{
				UE_LOGF(LogHAL, Verbose, "OpenSemaphore(AccessRights=0x%08x, bInherit=false, Name='%ls') returned ERROR_FILE_NOT_FOUND (LastError = %d)",
					AccessRights, Name, ErrNo);
			}
			else
			{
				UE_LOGF(LogHAL, Warning, "OpenSemaphore(AccessRights=0x%08x, bInherit=false, Name='%ls') failed with LastError = %d",
					AccessRights, Name, ErrNo);
			}

			return NULL;
		}
	}
	check(Semaphore);

	return new FWindowsSemaphore(Name, Semaphore);
}

bool FWindowsPlatformProcess::DeleteInterprocessSynchObject(FSemaphore * Object)
{
	if (NULL == Object)
	{
		return false;
	}

	FWindowsSemaphore * WinSem = static_cast< FWindowsSemaphore * >(Object);
	check( WinSem );

	HANDLE Semaphore = WinSem->GetSemaphore();
	bool bSucceeded = false;
	if (Semaphore)
	{
		bSucceeded = (CloseHandle(Semaphore) == TRUE);
		if (!bSucceeded)
		{
			DWORD ErrNo = GetLastError();
			UE_LOGF(LogHAL, Warning, "CloseHandle() for semaphore '%ls' failed with LastError = %d",
				Object->GetName(),
				ErrNo);
		}
	}

	// delete anyways
	delete WinSem;

	return bSucceeded;
}

bool FWindowsPlatformProcess::Daemonize()
{
	// TODO: implement
	return true;
}

void FWindowsPlatformProcess::SetupAudioThread()
{
	ensure(FPlatformMisc::CoInitialize());
}

void FWindowsPlatformProcess::TeardownAudioThread()
{
	FPlatformMisc::CoUninitialize();
}

/**
 * Maps a relative virtual address (RVA) to an address in memory.
 *
 * @param Header Pointer to the executable base
 * @param NtHeader Pointer to the NT header information in the image
 * @param Rva The RVA to to map
 *
 * @return Pointer to the data at this RVA
 */
static const void *MapRvaToPointer(const IMAGE_DOS_HEADER *Header, const IMAGE_NT_HEADERS *NtHeader, size_t Rva)
{
	const IMAGE_SECTION_HEADER *SectionHeaders = (const IMAGE_SECTION_HEADER*)(NtHeader + 1);
	for(size_t SectionIdx = 0; SectionIdx < NtHeader->FileHeader.NumberOfSections; SectionIdx++)
	{
		const IMAGE_SECTION_HEADER *SectionHeader = SectionHeaders + SectionIdx;
		if(Rva >= SectionHeader->VirtualAddress && Rva < SectionHeader->VirtualAddress + SectionHeader->SizeOfRawData)
		{
			return (const BYTE*)Header + SectionHeader->PointerToRawData + (Rva - SectionHeader->VirtualAddress);
		}
	}
	return NULL;
}

/**
 * Reads a list of import names from a portable executable file in memory.
 *
 * @param Header Pointer to the executable base
 * @param ImportNames Array to receive the list of imported PE file names
 */
static bool ReadLibraryImportsFromMemory(const IMAGE_DOS_HEADER *Header, TArray<FString> &ImportNames)
{
	bool bResult = false;
	if(Header->e_magic == IMAGE_DOS_SIGNATURE)
	{
		IMAGE_NT_HEADERS *NtHeader = (IMAGE_NT_HEADERS*)((BYTE*)Header + Header->e_lfanew);
		if(NtHeader->Signature == IMAGE_NT_SIGNATURE)
		{
			// Find the import directory header
			IMAGE_DATA_DIRECTORY *ImportDirectoryEntry = &NtHeader->OptionalHeader.DataDirectory[IMAGE_DIRECTORY_ENTRY_IMPORT];

			// Enumerate the imports
			IMAGE_IMPORT_DESCRIPTOR *ImportDescriptors = (IMAGE_IMPORT_DESCRIPTOR*)MapRvaToPointer(Header, NtHeader, ImportDirectoryEntry->VirtualAddress);
			for(size_t ImportIdx = 0; ImportIdx * sizeof(IMAGE_IMPORT_DESCRIPTOR) < ImportDirectoryEntry->Size; ImportIdx++)
			{
				IMAGE_IMPORT_DESCRIPTOR *ImportDescriptor = ImportDescriptors + ImportIdx;
				
				// "The end of the IMAGE_IMPORT_DESCRIPTOR array is indicated by an entry with fields all set to 0." -- https://docs.microsoft.com/en-us/archive/msdn-magazine/2002/march/inside-windows-an-in-depth-look-into-the-win32-portable-executable-file-format-part-2
				if(ImportDescriptor->Characteristics == 0 && ImportDescriptor->TimeDateStamp == 0 && ImportDescriptor->ForwarderChain == 0 && ImportDescriptor->Name == 0 && ImportDescriptor->FirstThunk == 0)
					break;

				if(ImportDescriptor->Name != 0)
				{
					const char *ImportName = (const char*)MapRvaToPointer(Header, NtHeader, ImportDescriptor->Name);
					ImportNames.Add(ImportName);
				}
			}

			bResult = true;
		}
	}
	return bResult;
}

/**
 * Reads a list of import names from a portable executable file.
 *
 * @param FileName Path to the library
 * @param ImportNames Array to receive the list of imported PE file names
 */
static bool ReadLibraryImports(const TCHAR* FileName, TArray<FString>& ImportNames)
{
	bool bResult = false;

	// Open the DLL using a file mapping, so we don't need to map any more than is necessary
	HANDLE NewFileHandle = CreateFile(FileName, GENERIC_READ, FILE_SHARE_READ, NULL, OPEN_EXISTING, FILE_ATTRIBUTE_NORMAL, NULL);
	if(NewFileHandle != INVALID_HANDLE_VALUE)
	{
		HANDLE NewFileMappingHandle = CreateFileMapping(NewFileHandle, NULL, PAGE_READONLY, 0, 0, NULL);
		if(NewFileMappingHandle != NULL)
		{
			void* NewData = MapViewOfFile(NewFileMappingHandle, FILE_MAP_READ, 0, 0, 0);
			if(NewData != NULL)
			{
				const IMAGE_DOS_HEADER* Header = (const IMAGE_DOS_HEADER*)NewData;
				bResult = ReadLibraryImportsFromMemory(Header, ImportNames);
				UnmapViewOfFile(NewData);
			}
			CloseHandle(NewFileMappingHandle);
		}
		CloseHandle(NewFileHandle);
	}

	return bResult;
}

bool FWindowsPlatformProcess::ResolveImport(const FString& Name, TConstArrayView<FString> SearchPaths, FString& OutFileName)
{
	IPlatformFile& PlatformFile = IPlatformFile::GetPlatformPhysical();
	auto SearchPathsFunc = [&PlatformFile, &OutFileName, &SearchPaths, &Name](int StartIdx, int EndIdx)
	{
		for (int Idx = StartIdx; Idx < EndIdx; Idx++)
		{
			TStringBuilder<MAX_PATH> FileName;
			FPathViews::Append(FileName, SearchPaths[Idx], Name);
			if (PlatformFile.FileExists(*FileName))
			{
				OutFileName = FPaths::ConvertRelativePathToFull(*FileName);
				return true;
			}
		}
		return false;
	};

	// Search the module and current dll directories found in the search path array first.
	// Note that there is an assumption that the first slots in the array are the module and
	// current dll directories.
	const int FirstAddedSearchPathIdx = 1 + ((DllDirectoryStack.Num() > 0) ? 1 : 0);
	if (SearchPathsFunc(0, FirstAddedSearchPathIdx))
	{
		return true;
	}

	// Search the dll cache that gets populated by AddDllDirectory
	FName DllName(*Name, FNAME_Find);
	if (DllName != NAME_None)
	{
		if (TArray<FString>* CachedPaths = SearchPathDllCache.Find(DllName))
		{
			for (auto Itr = CachedPaths->CreateIterator(); Itr; ++Itr)
			{
				const FString& FoundPath = *Itr;
				// Double check the dll still exists
				if (PlatformFile.FileExists(*FoundPath))
				{
					OutFileName = FoundPath;
					return true;
				}
				else
				{
					// The dll cache is out of date
					Itr.RemoveCurrent();
				}
			}

			// Remove invalid entry
			if (CachedPaths->Num() == 0)
			{
				SearchPathDllCache.Remove(DllName);
			}
		}
	}

	// Fall back to going through the search paths
	if (SearchPathsFunc(FirstAddedSearchPathIdx, SearchPaths.Num()))
	{
		return true;
	}

	return false;
}

#if CPUPROFILERTRACE_ENABLED

UE_TRACE_EVENT_BEGIN(Cpu, ResolveMissingImports, NoSync)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, Name)
UE_TRACE_EVENT_END()

#endif // CPUPROFILERTRACE_ENABLED

void FWindowsPlatformProcess::ResolveMissingImportsRecursive(const FString& FileName, TConstArrayView<FString> SearchPaths, TArray<FString>& ImportFileNames, TSet<FString>& VisitedImportNames)
{
#if CPUPROFILERTRACE_ENABLED
	UE_TRACE_LOG_SCOPED_T(Cpu, ResolveMissingImports, CpuChannel)
		<< ResolveMissingImports.Name(*FileName);
#endif // CPUPROFILERTRACE_ENABLED

	// Read the imports for this library
	TArray<FString> ImportNames;
	if(ReadLibraryImports(*FileName, ImportNames))
	{
		// Find all the imports that haven't already been resolved
		for(int Idx = 0; Idx < ImportNames.Num(); Idx++)
		{
			const FString &ImportName = ImportNames[Idx];
			if(!VisitedImportNames.Contains(ImportName))
			{
				// Prevent checking this import again
				VisitedImportNames.Add(ImportName);

				// Try to resolve this import
				if(GetModuleHandle(*ImportName) == NULL)
				{
					FString ImportFileName;
					if(ResolveImport(*ImportName, SearchPaths, ImportFileName))
					{
						ResolveMissingImportsRecursive(ImportFileName, SearchPaths, ImportFileNames, VisitedImportNames);
						ImportFileNames.Add(ImportFileName);
					}
				}
			}
		}
	}
}

/**
 * Log diagnostic messages showing missing imports for module.
 *
 * @param FileName Path to the library to load
 * @param SearchPaths Search directories to scan for imports
 */
static void LogImportDiagnostics(const FString& FileName, TConstArrayView<FString> SearchPaths)
{
	TArray<FString> ImportNames;
	if(ReadLibraryImports(*FileName, ImportNames))
	{
		bool bIncludeSearchPaths = false;
		for(const FString& ImportName : ImportNames)
		{
			if(GetModuleHandle(*ImportName) == nullptr)
			{
				UE_LOGF(LogWindows, Log, "  Missing import: %ls", *ImportName);
				bIncludeSearchPaths = true;
			}
		}
		if(bIncludeSearchPaths)
		{
			for (const FString& SearchPath : SearchPaths)
			{
				UE_LOGF(LogWindows, Log, "  Looked in: %ls", *SearchPath);
			}
		}
	}
}

#if CPUPROFILERTRACE_ENABLED

UE_TRACE_EVENT_BEGIN(Cpu, Windows_LoadLibrary, NoSync)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, Name)
UE_TRACE_EVENT_END()

#endif // CPUPROFILERTRACE_ENABLED

void* FWindowsPlatformProcess::LoadLibraryWithSearchPaths(const FString& FileName, TConstArrayView<FString> SearchPaths)
{
	UE_SCOPED_IO_ACTIVITY(*WriteToString<256>("Loading Dll ", FileName));

	// Make sure the initial module exists. If we can't find it from the path we're given, it's probably a system dll.
	FString FullFileName = FileName;
	IPlatformFile& PlatformFile = IPlatformFile::GetPlatformPhysical();
	if (PlatformFile.FileExists(*FullFileName))
	{
		// Convert it to a full path, since LoadLibrary will try to resolve it against the executable directory (which may not be the same as the working dir)
		FullFileName = FPaths::ConvertRelativePathToFull(FullFileName);

		// If this library is already loaded then just return now with the handle
		if (void* Handle = GetModuleHandle(*FullFileName))
		{
			return Handle;
		}

		// Create a list of files which we've already checked for imports. Don't add the initial file to this list to improve the resolution of dependencies for direct circular dependencies of this
		// module; by allowing the module to be visited twice, any mutually depended on DLLs will be visited first.
		TSet<FString> VisitedImportNames;

		// Find a list of all the DLLs that need to be loaded
		TArray<FString> ImportFileNames;
		ResolveMissingImportsRecursive(*FullFileName, SearchPaths, ImportFileNames, VisitedImportNames);

		TArray<TTuple<FString, DWORD>> ImportFailed;
 
		// Load all the missing dependencies first
		for (const FString& ImportFileName : ImportFileNames)
		{
			if (!GetModuleHandle(*ImportFileName))
			{
				const void* DependencyHandle = [&ImportFileName]() 
				{
#if CPUPROFILERTRACE_ENABLED
					UE_TRACE_LOG_SCOPED_T(Cpu, Windows_LoadLibrary, CpuChannel)
						<< Windows_LoadLibrary.Name(*ImportFileName);
#endif // CPUPROFILERTRACE_ENABLED
					return LoadLibrary(*ImportFileName);
				}();
				
				if (DependencyHandle)
				{
					UE_LOGF(LogWindows, Verbose, "Preloaded '%ls'", *ImportFileName);
				}
				else
				{
					ImportFailed.Add({ImportFileName, GetLastError()});
				}
			}
		}

		for (const auto& Import : ImportFailed)
		{
			if (!GetModuleHandle(*Import.Key))
			{
				UE_LOGF(LogWindows, Log, "Failed to preload '%ls' (GetLastError=%d)", *Import.Key, Import.Value);
				LogImportDiagnostics(Import.Key, SearchPaths);
			}
		}
	}

	// Try to load the actual library
	void* Handle = [&FullFileName]() 
	{
#if CPUPROFILERTRACE_ENABLED
		UE_TRACE_LOG_SCOPED_T(Cpu, Windows_LoadLibrary, CpuChannel)
			<< Windows_LoadLibrary.Name(*FullFileName);
#endif // CPUPROFILERTRACE_ENABLED
		return LoadLibrary(*FullFileName);
	}();
	
	if(Handle)
	{
		UE_LOGF(LogWindows, Verbose, "Loaded %ls", *FullFileName);
	}
	else
	{
		UE_LOGF(LogWindows, Log, "Failed to load '%ls' (GetLastError=%d)", *FileName, ::GetLastError());
		if (PlatformFile.FileExists(*FileName))
		{
			LogImportDiagnostics(FileName, SearchPaths);
		}
		else
		{
			UE_LOGF(LogWindows, Log, "File '%ls' does not exist", *FileName);
		}
	}
	return Handle;
}

FWindowsPlatformProcess::FProcEnumerator::FProcEnumerator()
{
	SnapshotHandle = ::CreateToolhelp32Snapshot(TH32CS_SNAPPROCESS, 0);
	CurrentEntry = new PROCESSENTRY32();
	CurrentEntry->dwSize = 0;
}

FWindowsPlatformProcess::FProcEnumerator::~FProcEnumerator()
{
	::CloseHandle(SnapshotHandle);
	delete CurrentEntry;
}

FWindowsPlatformProcess::FProcEnumInfo::FProcEnumInfo(const PROCESSENTRY32& InInfo)
	: Info(new PROCESSENTRY32(InInfo))
{

}

FWindowsPlatformProcess::FProcEnumInfo::~FProcEnumInfo()
{
	delete Info;
}

bool FWindowsPlatformProcess::FProcEnumerator::MoveNext()
{
	if (CurrentEntry->dwSize == 0)
	{
		CurrentEntry->dwSize = sizeof(PROCESSENTRY32);

		return ::Process32First(SnapshotHandle, CurrentEntry) == TRUE;
	}

	return ::Process32Next(SnapshotHandle, CurrentEntry) == TRUE;
}

FWindowsPlatformProcess::FProcEnumInfo FWindowsPlatformProcess::FProcEnumerator::GetCurrent() const
{
	return FProcEnumInfo(*CurrentEntry);
}

uint32 FWindowsPlatformProcess::FProcEnumInfo::GetPID() const
{
	return Info->th32ProcessID;
}

uint32 FWindowsPlatformProcess::FProcEnumInfo::GetParentPID() const
{
	return Info->th32ParentProcessID;
}

FString FWindowsPlatformProcess::FProcEnumInfo::GetName() const
{
	return Info->szExeFile;
}

FString FWindowsPlatformProcess::FProcEnumInfo::GetFullPath() const
{
	return GetApplicationName(GetPID());
}

void FWindowsPlatformProcess::SetupGameThread()
{
	SetThreadName(TEXT("GameThread"));
}

bool FWindowsPlatformProcess::IsFirstInstance()
{
	// Named mutex we use to figure out whether we are the first instance of the game running. This is needed to e.g.
	// make sure there is no contention when trying to save the shader cache.
	static bool bIsFirstInstance = WindowsPlatformProcess::MakeNamedMutex(FCommandLine::Get());
	return bIsFirstInstance;
}

void FWindowsPlatformProcess::CeaseBeingFirstInstance()
{
	// Release the mutex in the error case to ensure subsequent runs don't find it.
	WindowsPlatformProcess::ReleaseNamedMutex();
}

bool FWindowsPlatformProcess::TryGetMemoryUsage(FProcHandle& ProcessHandle, FPlatformProcessMemoryStats& OutStats)
{
	PROCESS_MEMORY_COUNTERS ProcessMemoryCounters;
	FPlatformMemory::Memzero(&ProcessMemoryCounters, sizeof(ProcessMemoryCounters));
	if (!::GetProcessMemoryInfo(ProcessHandle.Get(), &ProcessMemoryCounters, sizeof(ProcessMemoryCounters)))
	{
		if (FWindowsPlatformMisc::IsWine() && ::GetLastError() == ERROR_ACCESS_DENIED)
		{
			// Under Wine, GetProcessMemoryInfo() incorrectly returns ERROR_ACCESS_DENIED if a process is no longer running.
		}
		else
		{
			UE_LOGF(LogWindows, Warning, "Failure in call to GetProcessMemoryInfo (GetLastError=%d)", ::GetLastError());
		}
		return false;
	}
	OutStats.UsedPhysical = ProcessMemoryCounters.WorkingSetSize;
	OutStats.PeakUsedPhysical = ProcessMemoryCounters.PeakWorkingSetSize;
	OutStats.UsedVirtual = ProcessMemoryCounters.PagefileUsage;
	OutStats.PeakUsedVirtual = ProcessMemoryCounters.PeakPagefileUsage;
	return true;
}

static bool InitShouldExpectLowIntegrityLevel()
{
	// Set default based on preprocessor flag, but the behavior can be overridden on the command line at runtime.
	bool Ret = WINDOWS_LOWINTEGRITYLEVEL_EXPECT_DEFAULT;

	// -ExpectLowIntegrityLevel: Force low integrity level config.
	// -ExpectMediumIntegrityLevel: Force Windows default medium integrity level config.
	// If both options are specified then low integrity is selected.
	if (FParse::Param(::GetCommandLineW(), TEXT("ExpectLowIntegrityLevel")))
	{
		Ret = true;
	}
	else if (FParse::Param(::GetCommandLineW(), TEXT("ExpectMediumIntegrityLevel")))
	{
		Ret = false;
	}
	return Ret;
}

bool FWindowsPlatformProcess::ShouldExpectLowIntegrityLevel()
{
	static bool bExpectLowIntegrityLevel = InitShouldExpectLowIntegrityLevel();
	return bExpectLowIntegrityLevel;
}

namespace WindowsPlatformProcessImpl
{
	static void SetThreadName(LPCSTR ThreadName)
	{
#if !PLATFORM_SEH_EXCEPTIONS_DISABLED
		/**
		 * Code setting the thread name for use in the debugger.
		 *
		 * http://msdn.microsoft.com/en-us/library/xcb2z8hs.aspx
		 */
		const uint32 MS_VC_EXCEPTION=0x406D1388;

		struct THREADNAME_INFO
		{
			uint32 dwType;		// Must be 0x1000.
			LPCSTR szName;		// Pointer to name (in user addr space).
			uint32 dwThreadID;	// Thread ID (-1=caller thread).
			uint32 dwFlags;		// Reserved for future use, must be zero.
		};

		THREADNAME_INFO ThreadNameInfo;
		ThreadNameInfo.dwType		= 0x1000;
		ThreadNameInfo.szName		= ThreadName;
		ThreadNameInfo.dwThreadID	= ::GetCurrentThreadId();
		ThreadNameInfo.dwFlags		= 0;

		__try
		{
			RaiseException( MS_VC_EXCEPTION, 0, sizeof(ThreadNameInfo)/sizeof(ULONG_PTR), (ULONG_PTR*)&ThreadNameInfo );
		}
		__except( EXCEPTION_EXECUTE_HANDLER )
		CA_SUPPRESS(6322)
		{
		}
#endif
	}

	static void SetThreadDescription(PCWSTR lpThreadDescription)
	{
		// SetThreadDescription is only available from Windows 10 version 1607 / Windows Server 2016
		//
		// So in order to be compatible with older Windows versions we probe for the API at runtime
		// and call it only if available.

		typedef HRESULT(WINAPI *SetThreadDescriptionFnPtr)(HANDLE hThread, PCWSTR lpThreadDescription);

	PRAGMA_DISABLE_CAST_FUNCTION_TYPE_MISMATCH_WARNINGS
		static SetThreadDescriptionFnPtr RealSetThreadDescription = (SetThreadDescriptionFnPtr) GetProcAddress(GetModuleHandle(TEXT("kernel32.dll")), "SetThreadDescription");
	PRAGMA_ENABLE_CAST_FUNCTION_TYPE_MISMATCH_WARNINGS

		if (RealSetThreadDescription)
		{
			RealSetThreadDescription(::GetCurrentThread(), lpThreadDescription);
		}
	}

	static void SetThreadPriority(int NewPriority)
	{
		::SetThreadPriority(::GetCurrentThread(), NewPriority);
	}
}

void FWindowsPlatformProcess::SetThreadName( const TCHAR* ThreadName )
{
	// We try to use the SetThreadDescription API where possible since this
	// enables thread names in crashdumps and ETW traces
	WindowsPlatformProcessImpl::SetThreadDescription(TCHAR_TO_WCHAR(ThreadName));

	WindowsPlatformProcessImpl::SetThreadName(TCHAR_TO_ANSI(ThreadName));
}

void FWindowsPlatformProcess::SetThreadPriority(EThreadPriority NewPriority)
{
	::SetThreadPriority(::GetCurrentThread(), FRunnableThreadWin::TranslateThreadPriority(NewPriority));
}

class FWindowsProcessSentinel : public IProcessSentinel
{
public:
	virtual ~FWindowsProcessSentinel()
	{
		// this will also delete the file
		CloseHandle(FileHandle);
	}

	virtual bool CreateSentinelFile(const TCHAR* Name, const FString& Contents) override
	{
		FString Filename = MakeFilename(Name);

		// if the file exists, another process is running, so can't create it
		if (IFileManager::Get().FileExists(*Filename))
		{
			return false;
		}
		
		// create the file in a way that will delete itself when the process closes
		IFileManager::Get().MakeDirectory(*FPaths::GetPath(Filename), true);
		DWORD Sharing = FILE_SHARE_READ | FILE_SHARE_DELETE;
		DWORD Flags = FILE_FLAG_DELETE_ON_CLOSE;
		FileHandle = CreateFile(*Filename, GENERIC_WRITE, Sharing, nullptr, CREATE_ALWAYS, Flags, nullptr);
		
		// write and flush the contents
		FTCHARToUTF8 Convert(*Contents);
		DWORD BytesWritten = 0;
		WriteFile(FileHandle, Convert.Get(), Convert.Length(), &BytesWritten, nullptr);
		FlushFileBuffers(FileHandle);
		return FileHandle != nullptr;
	}
	
	virtual bool DoesSentinelFileExistImpl(const TCHAR* Name, FString* OptionalOutContents) override
	{
		FString Filename = MakeFilename(Name);

		// must also open with FILE_SHARE_DELETE for the above FILE_FLAG_DELETE_ON_CLOSE
		DWORD Sharing = FILE_SHARE_DELETE | FILE_SHARE_WRITE;
		DWORD Flags = 0;
		HANDLE Handle = CreateFile(*Filename, GENERIC_READ, Sharing, nullptr, OPEN_EXISTING, Flags, nullptr);

		// if this fails to open, it's because the file didn't exist - if it existed with a process running, it would return a value
		if (Handle == INVALID_HANDLE_VALUE)
		{
			if (IFileManager::Get().FileExists(*Filename))
			{
				UE_LOGF(LogWindows, Error, "WindowsSentinel: Didn't expect the file to exist if the Handle is invalid!");
			}

			return false;
		}

		if (OptionalOutContents != nullptr)
		{
			// get the filesize, without using GetFileSize which was always returning 0 (possibly due ot the FILE_FLAG_DELETE_ON_CLOSE)
			DWORD FileSize = SetFilePointer(Handle, 0, nullptr, FILE_END);
			SetFilePointer(Handle, 0, nullptr, FILE_BEGIN);

			// read in the contents
			ANSICHAR* Buf = (ANSICHAR*)FMemory::Malloc(FileSize + 1);
			if (ReadFile(Handle, Buf, FileSize, nullptr, nullptr))
			{
				Buf[FileSize] = 0;
				*OptionalOutContents = FString(Buf);
			}
			FMemory::Free(Buf);

		}

		CloseHandle(Handle);
		return true;
	}
	
	HANDLE FileHandle;
};

TSharedPtr<IProcessSentinel> FWindowsPlatformProcess::CreateProcessSentinelObject()
{
	return TSharedPtr<IProcessSentinel>(new FWindowsProcessSentinel());
}

PRAGMA_RESTORE_UNSAFE_TYPECAST_WARNINGS

#include "Windows/HideWindowsPlatformTypes.h"
