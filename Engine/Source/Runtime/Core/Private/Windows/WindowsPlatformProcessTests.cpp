// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/Build.h"

#if WITH_LOW_LEVEL_TESTS && PLATFORM_WINDOWS && !(defined(DISABLE_CWD_CHANGES) && DISABLE_CWD_CHANGES != 0)

#include "Containers/UnrealString.h"
#include "Windows/WindowsPlatformProcess.h"

#include "TestHarness.h"
#include "TestMacros/Assertions.h"

TEST_CASE("System::Core::Windows::FWindowsPlatformProcess::SetCurrentWorkingDirectoryToBaseDir",
          "[Core][Windows]")
{
	const FString OriginalBaseDir(FWindowsPlatformProcess::BaseDir());

	SECTION("Invalid base directory fires the expected verifyf")
	{
		FWindowsPlatformProcess::SetBaseDirForTesting(TEXT("C:/ThisPathDoesNotExistForLowLevelTestCoverage/"));

		REQUIRE_CHECK_MSG("Failed to set the working directory",
			FWindowsPlatformProcess::SetCurrentWorkingDirectoryToBaseDir());
	}

	FWindowsPlatformProcess::SetBaseDirForTesting(*OriginalBaseDir);
}

#endif
