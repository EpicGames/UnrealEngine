// Copyright Epic Games, Inc. All Rights Reserved.

#include "Windows/WindowsPlatformStackWalk.h"
#include "Async/RecursiveWordMutex.h"
#include "Async/UniqueLock.h"
#include "HAL/PlatformMemory.h"
#include "HAL/PlatformMisc.h"
#include "HAL/PlatformMutex.h"
#include "Logging/LogMacros.h"
#include "Math/UnrealMathUtility.h"
#include "HAL/UnrealMemory.h"
#include "Containers/StringConv.h"
#include "Containers/UnrealString.h"
#include "UObject/NameTypes.h"
#include "Misc/ScopeLock.h"
#include "Misc/Paths.h"
#include "Misc/CommandLine.h"
#include "Templates/UnrealTemplate.h"
#include "HAL/PlatformProcess.h"
#include "CoreGlobals.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/OutputDeviceRedirector.h"

#include "Windows/AllowWindowsPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
	#include <DbgHelp.h>
	#include <Shlwapi.h>
	#include <TlHelp32.h>
	#include <Psapi.h>
THIRD_PARTY_INCLUDES_END
#include "Windows/HideWindowsPlatformTypes.h"

#include "Modules/ModuleManager.h"


/*-----------------------------------------------------------------------------
	Stack walking.
	@TODO To be removed
-----------------------------------------------------------------------------*/

//@note Global lock as a tentative fix to narrow down UE-192420
static UE::FRecursiveWordMutex GStackWalkingLock;

// Set while a stack walk is in progress on the current thread. Without this guard,
// a failure inside the stack walker can recurse until the stack is exhausted.
static thread_local bool GStackWalkInProgress = false;

// Returns true if it is safe to start a stack walk on the current thread. Returns false
// if one is already in progress (the caller should bail out); in that case, an optional
// caller-supplied buffer is populated with a short marker so its output path is not
// left with stale data.
static bool CanProcessStackWalk(ANSICHAR* HumanReadableString = nullptr, SIZE_T HumanReadableStringSize = 0)
{
	if (GStackWalkInProgress)
	{
		if (HumanReadableString && HumanReadableStringSize > 0)
		{
			FCStringAnsi::Strncpy(HumanReadableString, "[stack walker re-entered; skipping]\n", HumanReadableStringSize);
		}
		return false;
	}
	return true;
}

/** Whether appInitStackWalking() has been called successfully or not. */
static bool GStackWalkingInitialized = false;
static bool GNeedToRefreshSymbols = false;
static HANDLE GProcessHandle = INVALID_HANDLE_VALUE;

static const TCHAR* CrashReporterSettings = TEXT("/Script/UnrealEd.CrashReporterSettings");

// NOTE: Make sure to enable Stack Frame pointers: bOmitFramePointers = false, or /Oy-
#define USE_FAST_STACKTRACE 1

// Uses StackWalk64 interface which is more reliable, but 500-1000x slower than the fast stacktrace.
#define USE_SLOW_STACKTRACE 0

// Use _NT_SYMBOL_PATH for non development builds. We don't want shipping crash reporter to try to 
// access build servers for example.
#if !UE_BUILD_SHIPPING
#define ALLOW_UNREAL_ACCESS_TO_NT_SYMBOL_PATH 1
#else
#define ALLOW_UNREAL_ACCESS_TO_NT_SYMBOL_PATH 0
#endif

// Optimization that only loads symbols on demand rather than always loading all symbols for loaded modules.
#if 1
#define ON_DEMAND_SYMBOL_LOADING 1
#else
#define ON_DEMAND_SYMBOL_LOADING 0
#endif

typedef bool  (WINAPI *TFEnumProcesses)( uint32* lpidProcess, uint32 cb, uint32* cbNeeded);
typedef bool  (WINAPI *TFEnumProcessModules)(HANDLE hProcess, HMODULE *lphModule, uint32 cb, LPDWORD lpcbNeeded);
typedef uint32 (WINAPI *TFGetModuleBaseName)(HANDLE hProcess, HMODULE hModule, LPWSTR lpBaseName, uint32 nSize);
typedef uint32 (WINAPI *TFGetModuleFileNameEx)(HANDLE hProcess, HMODULE hModule, LPWSTR lpFilename, uint32 nSize);
typedef bool  (WINAPI *TFGetModuleInformation)(HANDLE hProcess, HMODULE hModule, LPMODULEINFO lpmodinfo, uint32 cb);

static TFEnumProcesses			FEnumProcesses;
static TFEnumProcessModules		FEnumProcessModules;
static TFGetModuleBaseName		FGetModuleBaseName;
static TFGetModuleFileNameEx	FGetModuleFileNameEx;
static TFGetModuleInformation	FGetModuleInformation;

FString GetRemoteStorage(const FString& DownstreamStorage);
FString GetSymbolSearchPath();
HMODULE* GetProcessModules(HANDLE ProcessHandle);
void LoadSymbolsForModule(HMODULE ModuleHandle, const FString& RemoteStorage);
void LoadSymbolsForProcessModules(const FString &RemoteStorage);
void LoadSymbolsForModuleByAddress(uint64 Address, const FString& RemoteStorage, bool bShouldReloadModuleMissingSymbols);

struct FWindowsThreadContextWrapper
{
	enum 
	{
		MAGIC_VAL = 0x5397fe56
	};

	int32 Magic;
	CONTEXT	Context;
	HANDLE ThreadHandle;

	FWindowsThreadContextWrapper()
		: Magic(MAGIC_VAL)
		, Context{}
	{
	}
	void CheckOk()
	{
		check(Magic == MAGIC_VAL);
	}
};

#if USE_FAST_STACKTRACE
NTSYSAPI uint16 NTAPI RtlCaptureStackBackTrace(
	__in uint32 FramesToSkip,
	__in uint32 FramesToCapture,
	__out_ecount(FramesToCapture) PVOID *BackTrace,
	__out_opt PDWORD BackTraceHash
	);

/** Maximum callstack depth we support, no matter what OS we're running on. */
static constexpr ULONG GMaxCallstackDepth = 128;

#endif

UE_AUTORTFM_ALWAYS_OPEN void FWindowsPlatformStackWalk::StackWalkAndDump( ANSICHAR* HumanReadableString, SIZE_T HumanReadableStringSize, int32 IgnoreCount, void* Context )
{
	if (!CanProcessStackWalk(HumanReadableString, HumanReadableStringSize))
	{
		return;
	}
	TGuardValue<bool> ReentryGuard(GStackWalkInProgress, true);

	UE::TUniqueLock GlobalLock(GStackWalkingLock);

	InitStackWalking();

	// If the callstack is for the executing thread, ignore this function
	if (Context == nullptr)
	{
		IgnoreCount++;
	}
	FGenericPlatformStackWalk::StackWalkAndDump(HumanReadableString, HumanReadableStringSize, IgnoreCount, Context);
	// If we incremented IgnoreCount, make sure we have instructions after StackWalkAndDump so the compiler
	// can not remove this function from the callstack using Tail Call Elimination
	if (Context == nullptr)
	{
		static volatile int32 ForceCompilerToReturnHere = 0;
		ForceCompilerToReturnHere += static_cast<int32>(HumanReadableStringSize);
	}
}

UE_AUTORTFM_ALWAYS_OPEN void FWindowsPlatformStackWalk::StackWalkAndDump( ANSICHAR* HumanReadableString, SIZE_T HumanReadableStringSize, void* ProgramCounter, void* Context )
{
	if (!CanProcessStackWalk(HumanReadableString, HumanReadableStringSize))
	{
		return;
	}
	TGuardValue<bool> ReentryGuard(GStackWalkInProgress, true);

	UE::TUniqueLock GlobalLock(GStackWalkingLock);

	InitStackWalking();
	FGenericPlatformStackWalk::StackWalkAndDump(HumanReadableString, HumanReadableStringSize, ProgramCounter, Context);
}

UE_AUTORTFM_ALWAYS_OPEN TArray<FProgramCounterSymbolInfo> FWindowsPlatformStackWalk::GetStack(int32 IgnoreCount, int32 MaxDepth, void* Context)
{
	if (!CanProcessStackWalk())
	{
		return TArray<FProgramCounterSymbolInfo>();
	}
	TGuardValue<bool> ReentryGuard(GStackWalkInProgress, true);

	UE::TUniqueLock GlobalLock(GStackWalkingLock);

	InitStackWalking();

	// If the callstack is for the executing thread, ignore this function
	if(Context == nullptr)
	{
		IgnoreCount++;
	}
	return FGenericPlatformStackWalk::GetStack(IgnoreCount, MaxDepth, Context);
}

UE_AUTORTFM_ALWAYS_OPEN void FWindowsPlatformStackWalk::ThreadStackWalkAndDump(ANSICHAR* HumanReadableString, SIZE_T HumanReadableStringSize, int32 IgnoreCount, uint32 ThreadId)
{
	if (!CanProcessStackWalk(HumanReadableString, HumanReadableStringSize))
	{
		return;
	}
	TGuardValue<bool> ReentryGuard(GStackWalkInProgress, true);

	UE::TUniqueLock GlobalLock(GStackWalkingLock);

	InitStackWalking();
	HANDLE ThreadHandle = OpenThread(THREAD_GET_CONTEXT | THREAD_SET_CONTEXT | THREAD_TERMINATE | THREAD_SUSPEND_RESUME, false, ThreadId);
	if (ThreadHandle)
	{
		// sync with other threads that may try to suspend us as we try to suspend them
		static UE::FPlatformRecursiveMutex Mutex;
		UE::TUniqueLock Lock(Mutex);

		// Suspend the thread before grabbing its context (possible fix for incomplete callstacks)
		SuspendThread(ThreadHandle);

		// Give task scheduler some time to actually suspend the thread
		FPlatformProcess::Sleep(0.01f);

		FWindowsThreadContextWrapper ContextWapper;
		ContextWapper.Context.ContextFlags = CONTEXT_ALL;
		ContextWapper.ThreadHandle = ThreadHandle;

		if (GetThreadContext(ThreadHandle, &ContextWapper.Context))
		{
			FGenericPlatformStackWalk::StackWalkAndDump(HumanReadableString, HumanReadableStringSize, IgnoreCount, &ContextWapper);
		}

		ResumeThread(ThreadHandle);

		CloseHandle(ThreadHandle);
	}
}

static int32 CaptureStackTraceExternalProcess(uint64* BackTrace, uint32 MaxDepth, CONTEXT* InContext, HANDLE ThreadHandle, uint32* Depth)
{
	STACKFRAME64		StackFrame64;
	HANDLE				ProcessHandle = GProcessHandle;
	unsigned long		LastError;
	bool				bStackWalkSucceeded	= true;
	uint32				CurrentDepth		= 0;
	uint32				MachineType			= IMAGE_FILE_MACHINE_I386;
	CONTEXT				ContextCopy = *InContext;

	*Depth = 0;

#if !PLATFORM_SEH_EXCEPTIONS_DISABLED
	__try
#endif
	{
		// Zero out stack frame.
		FMemory::Memzero( StackFrame64 );

		// Initialize the STACKFRAME structure.
		StackFrame64.AddrPC.Mode         = AddrModeFlat;
		StackFrame64.AddrStack.Mode      = AddrModeFlat;
		StackFrame64.AddrFrame.Mode      = AddrModeFlat;
#if PLATFORM_CPU_ARM_FAMILY && !PLATFORM_WINDOWS_ARM64EC
		StackFrame64.AddrPC.Offset       = ContextCopy.Pc;
		StackFrame64.AddrStack.Offset    = ContextCopy.Sp;
		StackFrame64.AddrFrame.Offset    = ContextCopy.Fp;
		MachineType                      = IMAGE_FILE_MACHINE_ARM64;
#else
		StackFrame64.AddrPC.Offset       = ContextCopy.Rip;
		StackFrame64.AddrStack.Offset    = ContextCopy.Rsp;
		StackFrame64.AddrFrame.Offset    = ContextCopy.Rbp;
		MachineType                      = IMAGE_FILE_MACHINE_AMD64;
#endif

		// Walk the stack one frame at a time.
		while( bStackWalkSucceeded && (CurrentDepth < MaxDepth) )
		{
			bStackWalkSucceeded = !!StackWalk64(MachineType, 
												ProcessHandle, 
												ThreadHandle, 
												&StackFrame64,
												&ContextCopy,
												NULL,
												SymFunctionTableAccess64,
												SymGetModuleBase64,
												NULL );

			if( !bStackWalkSucceeded  )
			{
				// StackWalk failed! give up.
				LastError = GetLastError( );
				break;
			}

			// Stop if the frame pointer is NULL.
			// Note that the thread's PC 'StackFrame64.AddrPC.Offset' COULD be 0 in case something calls a nullptr.
			if( StackFrame64.AddrFrame.Offset == 0 )
			{
				break;
			}

			BackTrace[CurrentDepth++] = StackFrame64.AddrPC.Offset;
			*Depth = CurrentDepth;
		}
	} 
#if !PLATFORM_SEH_EXCEPTIONS_DISABLED
	__except( EXCEPTION_EXECUTE_HANDLER )
	{
		// We need to catch any exceptions within this function so they don't get sent to 
		// the engine's error handler, hence causing an infinite loop.
		return EXCEPTION_EXECUTE_HANDLER;
	} 
#endif

	// NULL out remaining entries.
	for ( ; CurrentDepth<MaxDepth; CurrentDepth++ )
	{
		BackTrace[CurrentDepth] = 0;
	}

	return EXCEPTION_EXECUTE_HANDLER;
}

void FWindowsPlatformStackWalk::CaptureStackTraceByProcess(uint64* OutBacktrace, uint32 MaxDepth, void* InContext, void* InThreadHandle, uint32* OutDepth, bool bExternalProcess)
{
	if (!bExternalProcess)
	{
		FMicrosoftPlatformStackWalk::CaptureStackTraceInternal(OutBacktrace, MaxDepth, InContext, InThreadHandle, OutDepth);
		// If we fail to capture a stack trace with this method, fall back to using dbghelp
		if (*OutDepth != 0)
		{
			return;
		}
	}

	// Init if not already initialized for another process
	InitStackWalking();
	CaptureStackTraceExternalProcess(OutBacktrace, MaxDepth, reinterpret_cast<PCONTEXT>(InContext), reinterpret_cast<HANDLE>(InThreadHandle), OutDepth);	
}

UE_AUTORTFM_ALWAYS_OPEN uint32 FWindowsPlatformStackWalk::CaptureThreadStackBackTrace(uint64 ThreadId, uint64* BackTrace, uint32 MaxDepth, void* Context)
{
	UE::TUniqueLock GlobalLock(GStackWalkingLock);

	if (BackTrace == nullptr || MaxDepth == 0)
	{
		return 0;
	}

	// Do not call InitStackWalking as we're just capturing addresses without using the dbghelp library etc if this is the same process.
	// If we're walking another process, they will already have initialized stackwalking to set GProcessHandle.
	// Don't suspend the calling thread, capture it's context directly and trace
	if (ThreadId == FPlatformTLS::GetCurrentThreadId())
	{
		// If we fail to capture a stack trace with this method, fall back to opening the thread handle
		uint32 Depth = CaptureStackBackTrace(BackTrace, MaxDepth);
		if (Depth != 0)
		{
			return Depth;
		}
	}

	HANDLE ThreadHandle = OpenThread(THREAD_GET_CONTEXT | THREAD_SET_CONTEXT | THREAD_TERMINATE | THREAD_SUSPEND_RESUME, false, (DWORD)ThreadId);
	if (!ThreadHandle)
	{
		return 0;
	}

	// Suspend the thread before grabbing its context
	if (SuspendThread(ThreadHandle) == (DWORD)-1)
	{
		return 0;
	}

	bool bExternalProcess = GProcessHandle != INVALID_HANDLE_VALUE && GProcessHandle != GetCurrentProcess();

	uint32 Depth = 0;
	if (Context)
	{
		FWindowsThreadContextWrapper* ContextWrapper = reinterpret_cast<FWindowsThreadContextWrapper*>(Context);
		ContextWrapper->ThreadHandle = ThreadHandle; // Use the thread handle open above to be sure it has the correct permissions.
		CaptureStackTraceByProcess(BackTrace, MaxDepth, &ContextWrapper->Context, ThreadHandle, &Depth, bExternalProcess);
	}
	else
	{
		CONTEXT ThreadContext;
		FMemory::Memzero(ThreadContext);
		ThreadContext.ContextFlags = CONTEXT_ALL;

		if (::GetThreadContext(ThreadHandle, &ThreadContext))
		{
			CaptureStackTraceByProcess(BackTrace, MaxDepth, &ThreadContext, ThreadHandle, &Depth, bExternalProcess);
		}
	}

	ResumeThread(ThreadHandle);

	CloseHandle(ThreadHandle);

	return Depth;
}

// #CrashReport: 2014-09-05 Switch to TArray<uint64,TFixedAllocator<100>>>
/**
 * Capture a stack backtrace and optionally use the passed in exception pointers.
 *
 * @param	BackTrace			[out] Pointer to array to take backtrace
 * @param	MaxDepth			Entries in BackTrace array
 * @param	Context				Optional thread context information (FWindowsThreadContextWrapper instance)
 */
UE_AUTORTFM_ALWAYS_OPEN uint32 FWindowsPlatformStackWalk::CaptureStackBackTrace( uint64* BackTrace, uint32 MaxDepth, void* Context )
{
	// Make sure we have place to store the information before we go through the process of raising
	// an exception and handling it.
	if (BackTrace == NULL || MaxDepth == 0)
	{
		return 0;
	}

	uint32 Depth = 0;
	if (Context)
	{
		FWindowsThreadContextWrapper* Wrapper = reinterpret_cast<FWindowsThreadContextWrapper*>(Context);
		FMicrosoftPlatformStackWalk::CaptureStackTraceInternal(BackTrace, MaxDepth, &Wrapper->Context, Wrapper->ThreadHandle, &Depth);
	}
	else
	{
#if USE_FAST_STACKTRACE
		PVOID WinBackTrace[GMaxCallstackDepth];
		ULONG RequestedDepth = FMath::Min<ULONG>(GMaxCallstackDepth, MaxDepth);
		uint16 NumFrames = RtlCaptureStackBackTrace(0, RequestedDepth, WinBackTrace, NULL);
		if (NumFrames == 0 && RequestedDepth > 0)
		{
			// RtlCaptureStackBackTrace sometimes returns 0, which NTAPI Undocumented Functions says is the value it returns to indicate an error
			// In all the cases we have tested, calling it again after it returns 0 will make it succeed and return non-zero
			NumFrames = RtlCaptureStackBackTrace(0, RequestedDepth, WinBackTrace, NULL);
			if (NumFrames == 0)
			{
				// Don't send a log since this might be called from inside the log system
				FPlatformMisc::LocalPrint(TEXT("RtlCaptureStackBackTrace failed; returning an empty backtrace."));
			}
		}
		Depth = NumFrames;
		for (uint16 FrameIndex = 0; FrameIndex < NumFrames; ++FrameIndex)
		{
			BackTrace[FrameIndex] = (uint64)WinBackTrace[FrameIndex];
		}
		while (NumFrames < MaxDepth)
		{
			BackTrace[NumFrames++] = 0;
		}		
#elif USE_SLOW_STACKTRACE
		// NOTE: Make sure to enable Stack Frame pointers: bOmitFramePointers = false, or /Oy-
		CONTEXT HelperContext;
		RtlCaptureContext(&HelperContext);

		FMicrosoftPlatformStackWalk::CaptureStackTraceInternal(BackTrace, MaxDepth, &HelperContext, GetCurrentThread(), &Depth);
#else
		// Raise an exception so CaptureStackBackTraceHelper has access to context record.
		__try
		{
			RaiseException(0, // Application-defined exception code.
				0,            // Zero indicates continuable exception.
				0,            // Number of arguments in args array (ignored if args is NULL)
				NULL);        // Array of arguments
		}
		// Capture the back trace.
		__except (FMicrosoftPlatformStackWalk::CaptureStackTraceInternal(BackTrace, MaxDepth, (GetExceptionInformation())->ContextRecord, GetCurrentThread(), &Depth))
		{
		}
#endif
	}	
	return Depth;
}

UE_AUTORTFM_ALWAYS_OPEN void FWindowsPlatformStackWalk::ProgramCounterToSymbolInfo( uint64 ProgramCounter, FProgramCounterSymbolInfo& OutSymbolInfo)
{
	const bool bIncludeInlineFrames = false;
	EnumerateSymbolInfosForProgramCounter(ProgramCounter, bIncludeInlineFrames, [&OutSymbolInfo](FProgramCounterSymbolInfo& SymbolInfo) { OutSymbolInfo = SymbolInfo; });
}

UE_AUTORTFM_ALWAYS_OPEN void FWindowsPlatformStackWalk::EnumerateSymbolInfosForProgramCounter(uint64 ProgramCounter, bool bIncludeInlineFrames, TFunctionRef<void (FProgramCounterSymbolInfo& OutSymbolInfo)> Callback)
{
	UE::TUniqueLock GlobalLock(GStackWalkingLock);

	// Initialize stack walking as it loads up symbol information which we require.
	InitStackWalking();

#if ON_DEMAND_SYMBOL_LOADING
	// Load symbols for the module
	bool bShouldReloadModuleMissingDebugSymbols = !FPlatformProperties::IsMonolithicBuild() && FPlatformStackWalk::WantsDetailedCallstacksInNonMonolithicBuilds();
	LoadSymbolsForModuleByAddress(ProgramCounter, GetSymbolSearchPath(), bShouldReloadModuleMissingDebugSymbols);
#endif

	FProgramCounterSymbolInfo SymbolInfo;
	// Set the program counter.
	SymbolInfo.ProgramCounter = ProgramCounter;

	HANDLE ProcessHandle = GProcessHandle;

	// Initialize symbol.
	ANSICHAR SymbolBuffer[sizeof( SYMBOL_INFO ) + FProgramCounterSymbolInfo::MAX_NAME_LENGTH] = {0};
	SYMBOL_INFO* Symbol = (SYMBOL_INFO*)SymbolBuffer;
	Symbol->SizeOfStruct = sizeof(SYMBOL_INFO);
	Symbol->MaxNameLen = FProgramCounterSymbolInfo::MAX_NAME_LENGTH;

	// Get module name.
	IMAGEHLP_MODULE64 ImageHelpModule = { 0 };
	ImageHelpModule.SizeOfStruct = sizeof(ImageHelpModule);
	if (SymGetModuleInfo64(ProcessHandle, ProgramCounter, &ImageHelpModule))
	{
		// Write out module information.
		FCStringAnsi::Strncpy(SymbolInfo.ModuleName, ImageHelpModule.ImageName, FProgramCounterSymbolInfo::MAX_NAME_LENGTH);
	}

	if (bIncludeInlineFrames)
	{
		// Check to see if we need to process inline frames
		DWORD NumInlineFrames = SymAddrIncludeInlineTrace(ProcessHandle, ProgramCounter);
		if (NumInlineFrames != 0)
		{
			DWORD InlineContext = 0; DWORD FrameIndex = 0;
			if (SymQueryInlineTrace(ProcessHandle, ProgramCounter, 0, ProgramCounter, ProgramCounter, &InlineContext, &FrameIndex))
			{
				for (DWORD Index = 0; Index < NumInlineFrames; ++Index, ++InlineContext)
				{
					DWORD64 Displacement = 0;
					if (SymFromInlineContext(ProcessHandle, ProgramCounter, InlineContext, &Displacement, Symbol))
					{
						FCStringAnsi::Strncpy(SymbolInfo.FunctionName, "[Inline Frame] ", FProgramCounterSymbolInfo::MAX_NAME_LENGTH);
						FCStringAnsi::StrncatTruncateDest(SymbolInfo.FunctionName, FProgramCounterSymbolInfo::MAX_NAME_LENGTH, Symbol->Name);
						FCStringAnsi::StrncatTruncateDest(SymbolInfo.FunctionName, FProgramCounterSymbolInfo::MAX_NAME_LENGTH, "()");

						SymbolInfo.OffsetInModule = ProgramCounter - Symbol->ModBase;

						IMAGEHLP_LINE64 Line{ 0 };
						Line.SizeOfStruct = sizeof(IMAGEHLP_LINE64);
						if (SymGetLineFromInlineContext(ProcessHandle, ProgramCounter, InlineContext, 0, (::DWORD*)&SymbolInfo.SymbolDisplacement, &Line))
						{
							SymbolInfo.LineNumber = Line.LineNumber;
							FCStringAnsi::Strncpy(SymbolInfo.Filename, Line.FileName, FProgramCounterSymbolInfo::MAX_NAME_LENGTH);
							Callback(SymbolInfo);
						}
					}
				}
			}
		}
	}

	// Get function name.
	if (SymFromAddr(ProcessHandle, ProgramCounter, nullptr, Symbol))
	{
		// Skip any funky chars in the beginning of a function name.
		int32 Offset = 0;
		while (Symbol->Name[Offset] < 32 || Symbol->Name[Offset] > 127)
		{
			Offset++;
		}
		
		SymbolInfo.OffsetInModule = ProgramCounter - Symbol->ModBase;

		// Write out function name.
		FCStringAnsi::Strncpy(SymbolInfo.FunctionName, Symbol->Name + Offset, FProgramCounterSymbolInfo::MAX_NAME_LENGTH);
		FCStringAnsi::StrncatTruncateDest(SymbolInfo.FunctionName, FProgramCounterSymbolInfo::MAX_NAME_LENGTH, "()");
	}

	// Get filename and line number.
	IMAGEHLP_LINE64	ImageHelpLine = {0};
	ImageHelpLine.SizeOfStruct = sizeof(ImageHelpLine);
	if (SymGetLineFromAddr64(ProcessHandle, ProgramCounter, (::DWORD*)&SymbolInfo.SymbolDisplacement, &ImageHelpLine))
	{
		CopyVirtualPathToLocal(SymbolInfo.Filename, FProgramCounterSymbolInfo::MAX_NAME_LENGTH, ImageHelpLine.FileName);
		SymbolInfo.LineNumber = ImageHelpLine.LineNumber;
	}

	Callback(SymbolInfo);
}

UE_AUTORTFM_ALWAYS_OPEN void FWindowsPlatformStackWalk::ProgramCounterToSymbolInfoEx(uint64 ProgramCounter, FProgramCounterSymbolInfoEx& OutSymbolInfoEx)
{
	FProgramCounterSymbolInfo SymbolInfo;
	ProgramCounterToSymbolInfo(ProgramCounter, SymbolInfo);

	OutSymbolInfoEx.ProgramCounter = ProgramCounter;
	OutSymbolInfoEx.LineNumber = SymbolInfo.LineNumber;
	OutSymbolInfoEx.SymbolDisplacement = SymbolInfo.SymbolDisplacement;
	OutSymbolInfoEx.OffsetInModule = SymbolInfo.OffsetInModule;
	OutSymbolInfoEx.FunctionName = StringCast<TCHAR, FProgramCounterSymbolInfo::MAX_NAME_LENGTH>(SymbolInfo.FunctionName);
	OutSymbolInfoEx.ModuleName = StringCast<TCHAR, FProgramCounterSymbolInfo::MAX_NAME_LENGTH>(SymbolInfo.ModuleName);
	OutSymbolInfoEx.Filename = StringCast<TCHAR, FProgramCounterSymbolInfo::MAX_NAME_LENGTH>(SymbolInfo.Filename);
}

/**
 * Get process module handle NULL-terminated list.
 * On error this method returns NULL.
 *
 * IMPORTANT: Returned value must be deallocated by FMemory::Free().
 */
HMODULE* GetProcessModules(HANDLE ProcessHandle)
{
	const int32 NumModules = FWindowsPlatformStackWalk::GetProcessModuleCount();
	// Allocate start size (last element reserved for NULL value)
	uint32   ResultBytes = NumModules * sizeof( HMODULE );
	HMODULE* ResultData = (HMODULE*)FMemory::Malloc( ResultBytes + sizeof( HMODULE ) );
	
	uint32 BytesRequired = 0;
	if (!FEnumProcessModules( ProcessHandle, ResultData, ResultBytes, (::DWORD *)&BytesRequired ))
	{
		FMemory::Free( ResultData );
		// Can't get process module list
		return nullptr;
	}
	if (BytesRequired <= ResultBytes)
	{
		// Add end module list marker
		ResultData[BytesRequired / sizeof( HMODULE )] = nullptr;
		return ResultData;
	}

	// No enough memory?
	return nullptr;
}

/** 
 * Upload locally built symbols to network symbol storage.
 *
 * Use case:
 *   Game designers use game from source (without prebuild game .dll-files).
 *   In this case all game .dll-files are compiled locally.
 *   For post-mortem debug programmers need .dll and .pdb files from designers.
 */
bool FWindowsPlatformStackWalk::UploadLocalSymbols()
{
	InitStackWalking();

	// Upload locally compiled files to symbol storage.
	FString SymbolStorage;
	if (!GConfig->GetString( CrashReporterSettings, TEXT( "UploadSymbolsPath" ), SymbolStorage, GEditorPerProjectIni ) || SymbolStorage.IsEmpty())
	{
		// Nothing to do.
		return true;
	}
	if (FParse::Param(FCommandLine::Get(), TEXT("DisableUploadSymbols")))
	{
		UE_LOGF(LogWindows, Log, "Uploading to symbol storage disabled by command line flag");
		return true;
	}
	// Prepare string
	SymbolStorage.ReplaceInline( TEXT( "/" ), TEXT( "\\" ), ESearchCase::CaseSensitive );
	SymbolStorage = TEXT( "SRV*" ) + SymbolStorage;

	int32 ErrorCode = 0;
	HANDLE ProcessHandle = GProcessHandle;

	// Enumerate process modules.
	HMODULE* ModuleHandlePointer = GetProcessModules( ProcessHandle );
	if (!ModuleHandlePointer)
	{
		ErrorCode = GetLastError();
		return false;
	}

#if WITH_EDITOR
	// Get Unreal Engine Editor directory for detecting non-game editor binaries.
	FString EnginePath = FPaths::ConvertRelativePathToFull( FPaths::EngineDir() );
	FPaths::MakePlatformFilename( EnginePath );
#endif

	// Upload all locally built modules.
	for (int32 ModuleIndex = 0; ModuleHandlePointer[ModuleIndex]; ModuleIndex++)
	{
		WCHAR ImageName[MAX_PATH] = {0};
		FGetModuleFileNameEx( ProcessHandle, ModuleHandlePointer[ModuleIndex], ImageName, MAX_PATH );

#if WITH_EDITOR
		WCHAR RelativePath[MAX_PATH];
		// Skip binaries inside Unreal Engine Editor directory (non-game editor binaries)
		if (PathRelativePathTo( RelativePath, *EnginePath, FILE_ATTRIBUTE_DIRECTORY, ImageName, 0 ) && FCString::Strncmp( RelativePath, TEXT( "..\\" ), 3 ))
		{
			continue;
		}
#endif

		WCHAR DebugName[MAX_PATH];
		FCString::Strcpy( DebugName, ImageName );

		if (PathRenameExtensionW( DebugName, L".pdb" ))
		{
			// Upload only if found .pdb file
			if (PathFileExistsW( DebugName ))
			{
				// Upload original file
				UE_LOGF( LogWindows, Log, "Uploading to symbol storage: %ls", ImageName );
				if (!SymSrvStoreFileW( ProcessHandle, *SymbolStorage, ImageName, SYMSTOREOPT_PASS_IF_EXISTS ))
				{
					HRESULT Result = GetLastError();
					TCHAR ErrorBuffer[1024];
					FPlatformMisc::GetSystemErrorMessage(ErrorBuffer, sizeof(ErrorBuffer) / sizeof(ErrorBuffer[0]), Result);
					UE_LOGF(LogWindows, Warning, "Uploading to symbol storage failed. Error Code %u: %ls", Result, ErrorBuffer);
					// Calling SymSrvStoreFileW can crash if called after failing, so ditch out of the loop on error
					break;
				}

				// Upload debug symbols
				UE_LOGF( LogWindows, Log, "Uploading to symbol storage: %ls", DebugName );
				if (!SymSrvStoreFileW( ProcessHandle, *SymbolStorage, DebugName, SYMSTOREOPT_PASS_IF_EXISTS ))
				{
					HRESULT Result = GetLastError();
					TCHAR ErrorBuffer[1024];
					FPlatformMisc::GetSystemErrorMessage(ErrorBuffer, sizeof(ErrorBuffer) / sizeof(ErrorBuffer[0]), Result);
					UE_LOGF(LogWindows, Warning, "Uploading to symbol storage failed. Error Code %u: %ls", Result, ErrorBuffer);
					// Calling SymSrvStoreFileW can crash if called after failing, so ditch out of the loop on error
					break;
				}
			}
		}
	}
	return true;
}

UE_AUTORTFM_ALWAYS_OPEN void LoadSymbolsForModule(HMODULE ModuleHandle, const FString& RemoteStorage)
{
	HANDLE ProcessHandle = GProcessHandle;

	int32 ErrorCode = 0;

	MODULEINFO ModuleInfo = { 0 };
	WCHAR ModuleName[FProgramCounterSymbolInfo::MAX_NAME_LENGTH] = { 0 };
	WCHAR ImageName[FProgramCounterSymbolInfo::MAX_NAME_LENGTH] = { 0 };
	static_assert(sizeof(MODULEINFO) == 24, "Broken alignment for 64bit Windows include.");
	FGetModuleInformation(ProcessHandle, ModuleHandle, &ModuleInfo, sizeof(ModuleInfo));
	FGetModuleFileNameEx(ProcessHandle, ModuleHandle, ImageName, FProgramCounterSymbolInfo::MAX_NAME_LENGTH);
	FGetModuleBaseName(ProcessHandle, ModuleHandle, ModuleName, FProgramCounterSymbolInfo::MAX_NAME_LENGTH);

	// Set the search path to find PDBs in the same folder as the DLL.
	WCHAR SearchPath[MAX_PATH] = { 0 };
	WCHAR* FileName = NULL;
	const auto Result = GetFullPathNameW(ImageName, MAX_PATH, SearchPath, &FileName);

	FString SearchPathList;
	if (Result != 0 && Result < MAX_PATH)
	{
		*FileName = 0;
		SearchPathList = SearchPath;
	}
	if (!RemoteStorage.IsEmpty())
	{
		if (!SearchPathList.IsEmpty())
		{
			SearchPathList.AppendChar(TEXT(';'));
		}
		SearchPathList.Append(RemoteStorage);
	}

	UE::TUniqueLock GlobalLock(GStackWalkingLock);

	SymSetSearchPathW(ProcessHandle, *SearchPathList);

	// Load module.
	const DWORD64 BaseAddress = SymLoadModuleExW(ProcessHandle, NULL, ImageName, ModuleName, (DWORD64)ModuleInfo.lpBaseOfDll, ModuleInfo.SizeOfImage, NULL, 0);
	if (!BaseAddress)
	{
		ErrorCode = GetLastError();

		// If the module is already loaded, the return value is zero and GetLastError returns ERROR_SUCCESS.
		if (ErrorCode != ERROR_SUCCESS)
		{
			TCHAR ErrorMsgBuffer[2048];
			const TCHAR* ErrorMsg = FWindowsPlatformMisc::GetSystemErrorMessage(ErrorMsgBuffer, UE_ARRAY_COUNT(ErrorMsgBuffer), ErrorCode);
			UE_LOGF(LogWindows, Verbose, "Failed to load module '%ls' with SymLoadModuleExW. Error %d: %ls", ModuleName, ErrorCode, ErrorMsg);
		}
	}
}


/**
 * Loads modules for current process.
 */
void LoadSymbolsForProcessModules(const FString &RemoteStorage)
{
	int32 ErrorCode = 0;
	HANDLE ProcessHandle = GProcessHandle;

	// Enumerate process modules.
	HMODULE* ModuleHandlePointer = GetProcessModules(ProcessHandle);
	if (!ModuleHandlePointer)
	{
		ErrorCode = GetLastError();
		return;
	}

	// Load the modules.
	for (int32 ModuleIndex = 0; ModuleHandlePointer[ModuleIndex]; ModuleIndex++)
	{
		LoadSymbolsForModule(ModuleHandlePointer[ModuleIndex], RemoteStorage);
	}

	// Free the module handle pointer allocated in case the static array was insufficient.
	FMemory::Free(ModuleHandlePointer);
}

UE_AUTORTFM_ALWAYS_OPEN void LoadSymbolsForModuleByAddress(uint64 Address, const FString& RemoteStorage, bool bShouldReloadModuleMissingDebugSymbols)
{
	UE::TUniqueLock GlobalLock(GStackWalkingLock);

	HMODULE ModuleHandle = NULL;

	if (GetModuleHandleEx(GET_MODULE_HANDLE_EX_FLAG_FROM_ADDRESS | GET_MODULE_HANDLE_EX_FLAG_UNCHANGED_REFCOUNT, (LPCTSTR)Address, &ModuleHandle))
	{
		// Check if the module was already loaded, but failed to locate the the debug symbols.
		if (bShouldReloadModuleMissingDebugSymbols)
		{
			IMAGEHLP_MODULE64 ImageHelpModule = {0};
			ImageHelpModule.SizeOfStruct = sizeof( ImageHelpModule );
			if (SymGetModuleInfo64(GProcessHandle, Address, &ImageHelpModule) && ImageHelpModule.SymType == SymNone)
			{
				MODULEINFO ModuleInfo = { 0 };
				FGetModuleInformation(GProcessHandle, ModuleHandle, &ModuleInfo, sizeof(ModuleInfo));

				// The module is already loaded but 'SymNone' means that we are missing debug symbols. The module was likely loaded implicitly while the symbol search path wasn't properly set, so the debug engine did not find the .pdb and
				// now that 'bad' state is cached. Unloading the module will clear the entry in the debug engine cache and loading it again with the proper symbol search path should pick up the .pdb this time.
				SymUnloadModule(GProcessHandle, (DWORD64)ModuleInfo.lpBaseOfDll);
			}
		}

		LoadSymbolsForModule(ModuleHandle, RemoteStorage);
	}
}


int32 FWindowsPlatformStackWalk::GetProcessModuleCount()
{
	FPlatformStackWalk::InitStackWalking();

	HANDLE ProcessHandle = GProcessHandle; 
	uint32 BytesRequired = 0;

	// Enumerate process modules.
	bool bEnumProcessModulesSucceeded = FEnumProcessModules( ProcessHandle, NULL, 0, (::DWORD *)&BytesRequired );
	if( !bEnumProcessModulesSucceeded )
	{
		return 0;
	}

	// Find out how many modules we need to load modules for.
	const int32 ModuleCount = BytesRequired / sizeof( HMODULE );
	return ModuleCount;
}

UE_AUTORTFM_ALWAYS_OPEN int32 FWindowsPlatformStackWalk::GetProcessModuleSignatures(FStackWalkModuleInfo *ModuleSignatures, const int32 ModuleSignaturesSize)
{
	UE::TUniqueLock GlobalLock(GStackWalkingLock);

	FPlatformStackWalk::InitStackWalking();

	HANDLE		ProcessHandle = GProcessHandle; 

	// Enumerate process modules.
	HMODULE* ModuleHandlePointer = GetProcessModules(ProcessHandle);
	if (!ModuleHandlePointer)
	{
		return 0;
	}

	// Find out how many modules we need to load modules for.
	IMAGEHLP_MODULEW64 Img = {0};
	Img.SizeOfStruct = sizeof(Img);

	int32 SignatureIndex = 0;

	// Load the modules.
	for( int32 ModuleIndex = 0; ModuleHandlePointer[ModuleIndex] && SignatureIndex < ModuleSignaturesSize; ModuleIndex++ )
	{
		MODULEINFO ModuleInfo = {0};
		WCHAR ModuleName[MAX_PATH] = {0};
		WCHAR ImageName[MAX_PATH] = {0};
		static_assert(sizeof( MODULEINFO ) == 24, "Broken alignment for 64bit Windows include.");
		FGetModuleInformation( ProcessHandle, ModuleHandlePointer[ModuleIndex], &ModuleInfo, sizeof( ModuleInfo ) );
		FGetModuleFileNameEx( ProcessHandle, ModuleHandlePointer[ModuleIndex], ImageName, MAX_PATH );
		FGetModuleBaseName( ProcessHandle, ModuleHandlePointer[ModuleIndex], ModuleName, MAX_PATH );

		FStackWalkModuleInfo Info = { 0 };

		// Load module and get rich image help information
		if (SymGetModuleInfoW64(ProcessHandle, (DWORD64)ModuleInfo.lpBaseOfDll, &Img))
		{

			Info.BaseOfImage = Img.BaseOfImage;
			FCString::Strcpy(Info.ImageName, Img.ImageName);
			Info.ImageSize = Img.ImageSize;
			FCString::Strcpy(Info.LoadedImageName, Img.LoadedImageName);
			FCString::Strcpy(Info.ModuleName, Img.ModuleName);
			Info.PdbAge = Img.PdbAge;
			Info.PdbSig = Img.PdbSig;
			FMemory::Memcpy(&Info.PdbSig70, &Img.PdbSig70, sizeof(GUID));
			Info.TimeDateStamp = Img.TimeDateStamp;
		}
		else
		{
			// Unable to get image help information, so fallback to the module info that is available
			Info.BaseOfImage = (uint64)ModuleInfo.lpBaseOfDll;
			FCString::Strcpy(Info.ImageName, ImageName);
			Info.ImageSize = ModuleInfo.SizeOfImage;
			FCString::Strcpy(Info.LoadedImageName, ImageName);

			FString BaseModuleName = FPaths::GetBaseFilename(ModuleName);
			FCString::Strncpy(Info.ModuleName, *BaseModuleName, 32);

			UE_LOGF(LogWindows, Verbose, "SymGetModuleInfoW64 failed, rich module information unavailable. Error Code: %u", GetLastError());
		}

		ModuleSignatures[SignatureIndex] = Info;
		++SignatureIndex;

	}

	// Free the module handle pointer allocated in case the static array was insufficient.
	FMemory::Free(ModuleHandlePointer);

	return SignatureIndex;
}

/**
 * Callback from the modules system that the loaded modules have changed and we need to reload symbols.
 */ 
static void OnModulesChanged( FName ModuleThatChanged, EModuleChangeReason ReasonForChange )
{
	// PluginDirectoryChanged didn't change which modules are loaded or unloaded.
	if (ReasonForChange == EModuleChangeReason::PluginDirectoryChanged)
	{
		return;
	}
#if ON_DEMAND_SYMBOL_LOADING
	if (ReasonForChange == EModuleChangeReason::ModuleLoaded)
	{
		return; // If the module is needed to resolve a callstack, the module and its symbols will be loaded on demand.
	}
#endif

	GNeedToRefreshSymbols = true;
}

/**
 * Find the downstream storage path.
 */
FString FWindowsPlatformStackWalk::GetDownstreamStorage()
{
	FString DownstreamStorage;
	if (GConfig->GetString(CrashReporterSettings, TEXT("DownstreamStorage"), DownstreamStorage, GEditorPerProjectIni) && !DownstreamStorage.IsEmpty())
	{
		DownstreamStorage = FPaths::ConvertRelativePathToFull(FPaths::RootDir(), DownstreamStorage);
	}
	else 
	{
		DownstreamStorage = FPaths::ConvertRelativePathToFull(FPaths::EngineIntermediateDir(), TEXT("Symbols"));
	}
	FPaths::MakePlatformFilename(DownstreamStorage);
	return DownstreamStorage;
}

void* FWindowsPlatformStackWalk::MakeThreadContextWrapper(void* Context, void* ThreadHandle)
{
	FWindowsThreadContextWrapper* ContextWrapper = new FWindowsThreadContextWrapper();
	ContextWrapper->Context = *((PCONTEXT)Context);
	ContextWrapper->ThreadHandle = (HANDLE)ThreadHandle;
	return ContextWrapper;
}

void FWindowsPlatformStackWalk::ReleaseThreadContextWrapper(void* ThreadContext)
{
	delete (FWindowsThreadContextWrapper*)ThreadContext;
}

/**
 * Create path symbol path for remote storage.
 */
FString GetRemoteStorage(const FString& DownstreamStorage)
{
	TArray<FString> RemoteStorage;
	GConfig->GetArray(CrashReporterSettings, TEXT("RemoteStorage"), RemoteStorage, GEditorPerProjectIni);
	if (RemoteStorage.Num() > 0)
	{
		FString SymbolStorage;
		for (int StorageIndex = 0; StorageIndex < RemoteStorage.Num(); ++StorageIndex)
		{
			if (StorageIndex > 0) 
			{
				SymbolStorage.AppendChar(TEXT(';'));
			}
			SymbolStorage.Append(TEXT("SRV*"));
			SymbolStorage.Append(DownstreamStorage);
			SymbolStorage.AppendChar(TEXT('*'));
			SymbolStorage.Append(RemoteStorage[StorageIndex]);
		}
		return SymbolStorage;
	}
	else
	{
		return FString();
	}
}

/**
 * Create path symbol path.
 * Reference: https://msdn.microsoft.com/en-us/library/ms681416%28v=vs.85%29.aspx?f=255&MSPPError=-2147217396
 */
FString GetSymbolSearchPath()
{
	FString SymbolSearchPath = GetRemoteStorage(FWindowsPlatformStackWalk::GetDownstreamStorage());
	if (SymbolSearchPath.IsEmpty())
	{
		// by default passing null to SymInitialize will use the current working dir to search for a pdb, 
		// but to support the basedir argument that allows an exe to run against data in a different location
		// we'll put the path of the executing module first.
		TCHAR ModulePath[MAX_PATH] = { 0 };
		if (::GetModuleFileName(::GetModuleHandle(NULL), ModulePath, MAX_PATH))
		{
			SymbolSearchPath = FPaths::GetPath(ModulePath);
			SymbolSearchPath += ";";
#if ALLOW_UNREAL_ACCESS_TO_NT_SYMBOL_PATH
			FString SymbolPathEnvironmentVariable = FPlatformMisc::GetEnvironmentVariable(L"_NT_SYMBOL_PATH");
			if (!SymbolPathEnvironmentVariable.IsEmpty())
			{
				SymbolSearchPath += SymbolPathEnvironmentVariable;
				SymbolSearchPath += ";";
			}
#endif
			SymbolSearchPath += FPlatformProcess::GetCurrentWorkingDirectory();
			SymbolSearchPath += ";";
		}
	}
	return SymbolSearchPath;
}



/**
 * Initializes the symbol engine if needed.
 */
UE_AUTORTFM_ALWAYS_OPEN bool FWindowsPlatformStackWalk::InitStackWalkingInternal(void* Process, bool bForceReinitOnProcessMismatch)
{
	UE::TUniqueLock GlobalLock(GStackWalkingLock);

	if (GStackWalkingInitialized)
	{
		if (Process == GProcessHandle && !GNeedToRefreshSymbols)
		{
			return true;
		}
		else if (Process != GProcessHandle) // Case: CrashReportClient walked its own process to log one of its ensure, then need to remote walk the Editor process to report an ensure/crash.
		{
			if (bForceReinitOnProcessMismatch)
			{
				SymCleanup(GProcessHandle);
				GStackWalkingInitialized = false;
				GNeedToRefreshSymbols = false;
				GProcessHandle = Process;
			}
			else
			{
				return false; // Stack walking is already initialized for another process.
			}
		}
		// Fallthrough.
	}
	else // Not initialized yet.
	{
		GProcessHandle = Process;
	}

	// Only do this code once, it never changes.
	static bool bHasRunOnce = false;
	if (!bHasRunOnce)
	{
		bHasRunOnce = true;
		void* DllHandle = FPlatformProcess::GetDllHandle( TEXT("PSAPI.DLL") );
		if( DllHandle == NULL )
		{
			return false;
		}

		// Load dynamically linked PSAPI routines.
		FEnumProcesses			= (TFEnumProcesses)			FPlatformProcess::GetDllExport( DllHandle,TEXT("EnumProcesses"));
		FEnumProcessModules		= (TFEnumProcessModules)	FPlatformProcess::GetDllExport( DllHandle,TEXT("EnumProcessModules"));
		FGetModuleFileNameEx	= (TFGetModuleFileNameEx)	FPlatformProcess::GetDllExport( DllHandle,TEXT("GetModuleFileNameExW"));
		FGetModuleBaseName		= (TFGetModuleBaseName)		FPlatformProcess::GetDllExport( DllHandle,TEXT("GetModuleBaseNameW"));
		FGetModuleInformation	= (TFGetModuleInformation)	FPlatformProcess::GetDllExport( DllHandle,TEXT("GetModuleInformation"));
	}

	// Abort if the required function pointers were not be loaded successfully.
	if( !FEnumProcesses || !FEnumProcessModules || !FGetModuleFileNameEx || !FGetModuleBaseName || !FGetModuleInformation )
	{
		return false;
	}

	auto InitDbgHelp = []()
	{
		// Set up the symbol engine.
		uint32 SymOpts = SymGetOptions();

		SymOpts |= SYMOPT_LOAD_LINES;
		SymOpts |= SYMOPT_FAIL_CRITICAL_ERRORS;
		SymOpts |= SYMOPT_DEFERRED_LOADS;
		SymOpts |= SYMOPT_EXACT_SYMBOLS;

		// This option allows for undecorated names to be handled by the symbol engine.
		SymOpts |= SYMOPT_UNDNAME;

		// Disable by default as it can be very spammy/slow.  Turn it on if you are debugging symbol look-up!
		//		SymOpts |= SYMOPT_DEBUG;

		// Not sure these are important or desirable
		//		SymOpts |= SYMOPT_ALLOW_ABSOLUTE_SYMBOLS;
		//		SymOpts |= SYMOPT_CASE_INSENSITIVE;

		SymSetOptions( SymOpts );

		// Initialize the symbol engine.
		FString SymbolSearchPath = GetSymbolSearchPath();
		SymInitializeW( GProcessHandle, SymbolSearchPath.IsEmpty() ? nullptr : *SymbolSearchPath, true );

		GNeedToRefreshSymbols = false;
		GStackWalkingInitialized = true;

#if !ON_DEMAND_SYMBOL_LOADING
		if (!FPlatformProperties::IsMonolithicBuild() && FPlatformStackWalk::WantsDetailedCallstacksInNonMonolithicBuilds())
		{
			const FString RemoteStorage = GetRemoteStorage(GetDownstreamStorage());
			LoadSymbolsForProcessModules(RemoteStorage);
		}
#endif
	};

	if (!GStackWalkingInitialized)
	{
		InitDbgHelp();
	}
	else if (GNeedToRefreshSymbols)
	{
#if ON_DEMAND_SYMBOL_LOADING
		// Cleaning up and reinitializing is faster than calling SymRefreshModuleList() in a basic test conditions. The MS documentation claims that SymInitialize() with bInvadeProcess true (as done above)
		// do the same thing as SymRefreshModuleList(), but that's likely not true. It was measured as following:
		//
		// FPlatformStackWalk::InitStackWalking(); // Must be the very first call (no prior ensure and anything that dumped the stack)
		// FPlatformStackWalk::StackWalkAndDump(...);
		// FModuleManager::Get().OnModulesChanged().Broadcast(NAME_None, EModuleChangeReason::ModuleUnloaded); // This set to GNeedToRefreshSymbols = true.
		// FPlatformStackWalk::InitStackWalking(); // This will trigger the code that needs to be measured.
		// FPlatformStackWalk::StackWalkAndDump(...);
		//
		// If we use: SymRefreshModuleList(), it takes 7 seconds and the memory usage of the Editor is doubled.
		// If we use: SymCleanup() + SymInitializeW(..., bInvadeProcess=true), it takes 0.03s and doesn't use more memory to perform the dump.
		//
		// Why? Hard to tell. Maybe a bug. That's in the internal working of DBGHelp and this might change in the future. Cleaning up DBGHelp
		// invalidates the cached states. This may affect negatively other use cases, but that's speculation. Also, module change events mostly
		// happen at startup, usually before PlatformStackWalk::InitStackWalking() is called the first time unless an ensure() fires early or
		// some code calls StackWalkAndDump() during the initialization phase, forcing initialization of DBGHelp.
		SymCleanup(GProcessHandle);
		InitDbgHelp();
#else
		// Refresh and reload symbols
		SymRefreshModuleList(GProcessHandle);

		GNeedToRefreshSymbols = false;

		if (!FPlatformProperties::IsMonolithicBuild() && FPlatformStackWalk::WantsDetailedCallstacksInNonMonolithicBuilds())
		{
			const FString RemoteStorage = GetRemoteStorage( GetDownstreamStorage() );
			// When refresh is needed we cannot track which modules have been loaded and are interesting
			// so load symbols for all modules the process has loaded.
			LoadSymbolsForProcessModules( RemoteStorage );
		}
#endif
	}

	return GStackWalkingInitialized;
}


bool FWindowsPlatformStackWalk::InitStackWalking()
{
	// When stack walking is already initialized, the code often implicitly assumes that the same process is going to
	// be walked, but that's not always true since CrashReportClient can walk its own process to log an ensure and walk
	// the Editor process to report an ensure/crash. If stack walking is already initialized, keep walking the same process.
	return FWindowsPlatformStackWalk::InitStackWalkingInternal(GetCurrentProcess(), /*bForceReinitOnProcessMismatch*/false);
}

bool FWindowsPlatformStackWalk::InitStackWalkingForProcess(const FProcHandle& Process)
{
	// When the caller specifies a process, that process become the 'official' one and this will reinitialize the stack walking
	// if the current process and the new process don't match. Using an invalid process handle means 'force to reinitialize to the current process'.
	return FWindowsPlatformStackWalk::InitStackWalkingInternal(Process.IsValid() ? Process.Get() : GetCurrentProcess(), /*bForceReinitOnProcessMismatch*/true);
}

void FWindowsPlatformStackWalk::RegisterOnModulesChanged()
{
	// Register for callback so we can reload symbols when new modules are loaded
	FModuleManager::Get().OnModulesChanged().AddStatic( &OnModulesChanged );
}

UE_AUTORTFM_ALWAYS_OPEN bool FWindowsPlatformStackWalk::GetFunctionDefinitionLocation(const FString& FunctionSymbolName, const FString& FunctionModuleName, FString& OutPathname, uint32& OutLineNumber, uint32& OutColumnNumber)
{
	UE::TUniqueLock GlobalLock(GStackWalkingLock);

#if ON_DEMAND_SYMBOL_LOADING
	bool bShouldReloadModuleMissingDebugSymbols = !FPlatformProperties::IsMonolithicBuild() && !FunctionModuleName.IsEmpty();
	if (bShouldReloadModuleMissingDebugSymbols)
	{
		HMODULE ModuleHandle = GetModuleHandle(*FunctionModuleName);
		if (!ModuleHandle)
		{
			return false;
		}

		MODULEINFO ModuleInfo = { 0 };
		FGetModuleInformation(GProcessHandle, ModuleHandle, &ModuleInfo, sizeof(ModuleInfo));

		// Get the module info containing the debug symbols state.
		IMAGEHLP_MODULE64 ImageHelpModule = {0};
		ImageHelpModule.SizeOfStruct = sizeof( ImageHelpModule );
		if (!SymGetModuleInfo64(GProcessHandle, (DWORD64)ModuleInfo.EntryPoint, &ImageHelpModule))
		{
			return false;
		}

		if (ImageHelpModule.SymType == SymNone)
		{
			// The module is already loaded but 'SymNone' means that we are missing debug symbols. The module was likely loaded implicitly while the symbol search path wasn't properly set, so the debug engine did not find the .pdb and
			// now that 'bad' state is cached. Unloading the module will clear the entry in the debug engine cache and loading it again with the proper symbol search path should pick up the .pdb this time.
			SymUnloadModule(GProcessHandle, (DWORD64)ModuleInfo.lpBaseOfDll);
		}

		if (ImageHelpModule.SymType == SymDeferred || ImageHelpModule.SymType == SymNone)
		{
			// Load (or reload the module with) the debug symbols.
			LoadSymbolsForModule(ModuleHandle, GetRemoteStorage(GetDownstreamStorage()));
		}
	}
#endif

	ANSICHAR SymbolInfoBuffer[sizeof(IMAGEHLP_SYMBOL64) + MAX_SYM_NAME];
	PIMAGEHLP_SYMBOL64 SymbolInfoPtr = reinterpret_cast<IMAGEHLP_SYMBOL64*>(SymbolInfoBuffer);
	SymbolInfoPtr->SizeOfStruct = sizeof(SymbolInfoBuffer);
	SymbolInfoPtr->MaxNameLength = MAX_SYM_NAME;

	FString FullyQualifiedSymbolName = FunctionSymbolName;
	if(!FunctionModuleName.IsEmpty())
	{
		FullyQualifiedSymbolName = FString::Printf(TEXT( "%s!%s" ), *FunctionModuleName, *FunctionSymbolName);
	}

	// Query information about this symbol by name
	if(::SymGetSymFromName64(GProcessHandle, TCHAR_TO_ANSI(*FullyQualifiedSymbolName ), SymbolInfoPtr))
	{
		IMAGEHLP_LINE64 FileAndLineInfo;
		FileAndLineInfo.SizeOfStruct = sizeof(FileAndLineInfo);

		// Query file and line number information for this symbol.
		if(::SymGetLineFromAddr64(GProcessHandle, SymbolInfoPtr->Address, (::DWORD*)&OutColumnNumber, &FileAndLineInfo))
		{
			OutPathname = (const ANSICHAR*)(FileAndLineInfo.FileName);
			OutLineNumber = FileAndLineInfo.LineNumber;
			return true;
		}
	}

	return false;
}
