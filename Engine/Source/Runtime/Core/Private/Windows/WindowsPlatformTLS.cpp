// Copyright Epic Games, Inc. All Rights Reserved.

#include "Windows/WindowsPlatformTLS.h"

#if WITH_EDITOR

#include "Templates/UnrealTemplate.h"
#include "Logging/LogMacros.h"
#include "Logging/StructuredLog.h"
#include "CoreGlobals.h"

#include <AtomicQueue.h>

// Custom cross-platform dynamic TLS implementation because we hit OS TLS slot limit on many platforms. 
// Compile-time defined num slot and num thread limits.
// Allocating and freeing slots can be contended, getting and setting a slot value is fast but has an additional indirection compared with OS TLS.

namespace WindowsPlatformTLS_Private
{
	uint32 PrimarySlot = FGenericPlatformTLS::InvalidTlsSlot;

	// lock-free container for indices in the range [0..Size)
	template<uint32 Size>
	class TIndices
	{
	public:
		TIndices()
		{
			// initially all indices are available
			for (uint32 i = 0; i != Size; ++i)
			{
				// `0` is a special value for AtomicQueue, shift away from it
				Queue.push(i + 1);
			}
		}

		uint32 Alloc()
		{
			return Queue.pop() - 1;
		}

		bool TryAlloc(uint32& OutIndex)
		{
			if (!Queue.try_pop(OutIndex))
			{
				return false;
			}
			OutIndex -= 1;
			return true;
		}

		void Free(uint32 Value)
		{
			Queue.push(Value + 1);
		}

	private:
		atomic_queue::AtomicQueue<uint32, Size> Queue;
	};

	// TLS storage for all threads and slots
	class FStorage
	{
	public:
		void** GetThreadStorage(uint32 ThreadIndex)
		{
			return Buffer[ThreadIndex];
		}

		uint32 GetThreadIndex(void** ThreadStorage)
		{
			uint32 ThreadIndex = uint32((ThreadStorage - *Buffer) / MaxSlots);
			check(ThreadIndex < MaxThreads);
			return ThreadIndex;
		}

		void ResetSlot(uint32 SlotIndex)
		{
			for (uint32 ThreadIndex = 0; ThreadIndex != MaxThreads; ++ThreadIndex)
			{
				Buffer[ThreadIndex][SlotIndex] = nullptr;
			}
		}

		void ResetThread(uint32 ThreadIndex)
		{
			FMemory::Memzero(Buffer[ThreadIndex], MaxSlots * sizeof(void*));
 		}

	private:
		void* Buffer[MaxThreads][MaxSlots]; // compile-time zero initialized
	};

	FStorage Storage;

	void OnThreadExit(void* TlsValue);

	// uses local static var to not depend on static initialization order
	class FSingleton
	{
	private:
		FSingleton()
		{
			PrimarySlot = Windows::FlsAlloc(&OnThreadExit);
		}

	public:
		static FSingleton& Get()
		{
			static FSingleton Singleton;
			return Singleton;
		}

		TIndices<MaxSlots> Slots;
		TIndices<MaxThreads> Threads;
	};

	// called on thread exit if the thread has ever set TLS value
	// TlsValue: a value set to PrimarySlot
	void OnThreadExit(void* TlsValue)
	{
		check(PrimarySlot < MaxSlots);

#if DO_CHECK
		void* TlsValueLocal = Windows::FlsGetValue(PrimarySlot);
		checkf(TlsValue == TlsValueLocal, TEXT("%p - %p"), TlsValue, TlsValueLocal);
#endif

		Windows::FlsSetValue(PrimarySlot, nullptr); // clear the value otherwise this function will be called again

		void** ThreadStorage = (void**)TlsValue;
		uint32 ThreadIndex = Storage.GetThreadIndex(ThreadStorage);
		Storage.ResetThread(ThreadIndex);
		FSingleton::Get().Threads.Free(ThreadIndex);
	}

	void** AllocThreadStorage()
	{
		uint32 ThreadIndex = FSingleton::Get().Threads.Alloc();
		void** ThreadStorage = Storage.GetThreadStorage(ThreadIndex);
		Windows::FlsSetValue(PrimarySlot, ThreadStorage);
		return ThreadStorage;
	}
}

uint32 FWindowsPlatformTLS::AllocTlsSlot()
{
	uint32 Result;
	if (WindowsPlatformTLS_Private::FSingleton::Get().Slots.TryAlloc(Result))
	{
		return Result;
	}
	UE_LOGFMT(LogCore, Fatal, "Maximum number of TlsSlots (WINDOWS_MAX_NUM_TLS_SLOTS == {MaxSlots}) exceeded when calling AllocTlsSlot.",
		WindowsPlatformTLS_Private::MaxSlots);
}

void FWindowsPlatformTLS::FreeTlsSlot(uint32 SlotIndex)
{
	using namespace WindowsPlatformTLS_Private;

	checkf(SlotIndex < MaxSlots, TEXT("Invalid TLS slot index %u"), SlotIndex);

	Storage.ResetSlot(SlotIndex);
	FSingleton::Get().Slots.Free(SlotIndex);
}

#endif