// Copyright Epic Games, Inc. All Rights Reserved.

#include "Windows/WindowsPlatformTime.h"
#include "Misc/AssertionMacros.h"
#include "Containers/Ticker.h"
#include "Windows/WindowsHWrapper.h"
#include "Stats/Stats.h"

float FWindowsPlatformTime::CPUTimePctRelative = 0.0f;

namespace FWindowsTimeInternal
{
	struct FThreadCPUStats
	{
		/** Current thread percentage CPU utilization for the last interval relative to one core. */
		float ThreadCPUTimePctRelative = 0.f;

		/** The per-thread CPU processing time (kernel + user) from the last update */
		double LastIntervalThreadTime = 0;
	};

	/** Per-Thread CPU Stats */
	thread_local FThreadCPUStats CurrentThreadCPUStats = {};
}


double FWindowsPlatformTime::InitTiming(void)
{
	LARGE_INTEGER Frequency;
	verify( QueryPerformanceFrequency(&Frequency) );
	SecondsPerCycle = 1.0 / (double)Frequency.QuadPart;
	SecondsPerCycle64 = 1.0 / (double)Frequency.QuadPart;

	// Due to some limitation of the OS, we limit the polling frequency to 4 times per second, 
	// but it should be enough for longterm CPU usage monitoring.
	static const float PollingInterval = 1.0f / 4.0f;

	// Register a ticker delegate for updating the CPU utilization data.
	FTSTicker::GetCoreTicker().AddTicker( FTickerDelegate::CreateStatic( &FPlatformTime::UpdateCPUTime ), PollingInterval );

	return FPlatformTime::Seconds();
}


void FWindowsPlatformTime::SystemTime( int32& Year, int32& Month, int32& DayOfWeek, int32& Day, int32& Hour, int32& Min, int32& Sec, int32& MSec )
{
	SYSTEMTIME st;
	GetLocalTime( &st );

	Year		= st.wYear;
	Month		= st.wMonth;
	DayOfWeek	= st.wDayOfWeek;
	Day			= st.wDay;
	Hour		= st.wHour;
	Min			= st.wMinute;
	Sec			= st.wSecond;
	MSec		= st.wMilliseconds;
}


void FWindowsPlatformTime::UtcTime( int32& Year, int32& Month, int32& DayOfWeek, int32& Day, int32& Hour, int32& Min, int32& Sec, int32& MSec )
{
	SYSTEMTIME st;
	GetSystemTime( &st );

	Year		= st.wYear;
	Month		= st.wMonth;
	DayOfWeek	= st.wDayOfWeek;
	Day			= st.wDay;
	Hour		= st.wHour;
	Min			= st.wMinute;
	Sec			= st.wSecond;
	MSec		= st.wMilliseconds;
}


/** Holds Windows filetime misc functions. */
struct FFiletimeMisc
{
	/**
	 * @return number of ticks based on the specified Filetime.
	 */
	static FORCEINLINE uint64 TicksFromFileTime( const FILETIME& Filetime )
	{
		const uint64 NumTicks = (uint64(Filetime.dwHighDateTime) << 32) + Filetime.dwLowDateTime;
		return NumTicks;
	}

	/**
	 * @return number of seconds based on the specified Filetime.
	 */
	static FORCEINLINE double ToSeconds( const FILETIME& Filetime )
	{
		return double(TicksFromFileTime( Filetime ))/double(ETimespan::TicksPerSecond);
	}
};


bool FWindowsPlatformTime::UpdateCPUTime( float /*DeltaTime*/ )
{
    QUICK_SCOPE_CYCLE_COUNTER(STAT_FWindowsPlatformTime_UpdateCPUTime);

	static double LastTotalProcessTime = 0.0f;
	static double LastTotalUserAndKernelTime = 0.0f;

	FILETIME CreationTime = {0};
	FILETIME ExitTime = {0};
	FILETIME KernelTime = {0};
	FILETIME UserTime = {0};
	FILETIME CurrentTime = {0};

	::GetProcessTimes( ::GetCurrentProcess(), &CreationTime, &ExitTime, &KernelTime, &UserTime );
	::GetSystemTimeAsFileTime( &CurrentTime );

	const double CurrentTotalUserAndKernelTime = FFiletimeMisc::ToSeconds(KernelTime) + FFiletimeMisc::ToSeconds(UserTime);
	const double CurrentTotalProcessTime = FFiletimeMisc::ToSeconds(CurrentTime)-FFiletimeMisc::ToSeconds(CreationTime);

	const double IntervalProcessTime = CurrentTotalProcessTime - LastTotalProcessTime;
	const double IntervalUserAndKernelTime = CurrentTotalUserAndKernelTime - LastTotalUserAndKernelTime;

	// IntervalUserAndKernelTime == 0.0f means that the OS hasn't updated the data yet, 
	// so don't update to avoid oscillating between 0 and calculated value.
	if( IntervalUserAndKernelTime > 0.0 )
	{
		CPUTimePctRelative = (float)(IntervalUserAndKernelTime/IntervalProcessTime * 100.0);

		LastTotalProcessTime = CurrentTotalProcessTime;
		LastTotalUserAndKernelTime = CurrentTotalUserAndKernelTime;
		LastIntervalCPUTimeInSeconds = IntervalUserAndKernelTime;
	}

	return true;
}

bool FWindowsPlatformTime::UpdateThreadCPUTime(float/*= 0.0*/)
{
	struct FThreadCPUTime
	{
		double LastTotalThreadTime = 0.0;
		double LastTotalThreadUserAndKernelTime = 0.0;
	};

	thread_local FThreadCPUTime ThreadTimeInfo = {};

	FILETIME ThreadCreationTime = {0};
	FILETIME ThreadExitTime = {0};
	FILETIME ThreadKernelTime = {0};
	FILETIME ThreadUserTime = {0};
	FILETIME CurrentTime = {0};

	::GetThreadTimes(::GetCurrentThread(), &ThreadCreationTime, &ThreadExitTime, &ThreadKernelTime, &ThreadUserTime);
	::GetSystemTimeAsFileTime(&CurrentTime);

	const double CurrentTotalThreadUserAndKernelTime = FFiletimeMisc::ToSeconds(ThreadKernelTime) + FFiletimeMisc::ToSeconds(ThreadUserTime);
	const double CurrentTotalThreadTime = FFiletimeMisc::ToSeconds(CurrentTime) - FFiletimeMisc::ToSeconds(ThreadCreationTime);

	const double IntervalThreadTime = CurrentTotalThreadTime - ThreadTimeInfo.LastTotalThreadTime;
	const double IntervalThreadUserAndKernelTime = CurrentTotalThreadUserAndKernelTime - ThreadTimeInfo.LastTotalThreadUserAndKernelTime;

	// IntervalUserAndKernelTime == 0.0f means that the OS hasn't updated the data yet, 
	// so don't update to avoid oscillating between 0 and calculated value.
	if (IntervalThreadUserAndKernelTime > 0.0)
	{
		FWindowsTimeInternal::CurrentThreadCPUStats.ThreadCPUTimePctRelative =
			(float)((IntervalThreadUserAndKernelTime / IntervalThreadTime) * 100.0);

		ThreadTimeInfo.LastTotalThreadTime = CurrentTotalThreadTime;
		ThreadTimeInfo.LastTotalThreadUserAndKernelTime = CurrentTotalThreadUserAndKernelTime;
		FWindowsTimeInternal::CurrentThreadCPUStats.LastIntervalThreadTime = IntervalThreadUserAndKernelTime;
	}

	return true;
}

void FWindowsPlatformTime::AutoUpdateGameThreadCPUTime(double UpdateInterval)
{
	static bool bEnabledGameThreadTiming = false;

	if (!bEnabledGameThreadTiming)
	{
		FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateStatic(&FPlatformTime::UpdateThreadCPUTime), (float)UpdateInterval);

		bEnabledGameThreadTiming = true;
	}
}

FCPUTime FWindowsPlatformTime::GetCPUTime()
{
	return FCPUTime( CPUTimePctRelative / (float)FPlatformMisc::NumberOfCoresIncludingHyperthreads(), CPUTimePctRelative );
}

FCPUTime FWindowsPlatformTime::GetThreadCPUTime()
{
	return FCPUTime(FWindowsTimeInternal::CurrentThreadCPUStats.ThreadCPUTimePctRelative /
						(float)FPlatformMisc::NumberOfCoresIncludingHyperthreads(),
					FWindowsTimeInternal::CurrentThreadCPUStats.ThreadCPUTimePctRelative);
}

double FWindowsPlatformTime::GetLastIntervalThreadCPUTimeInSeconds()
{
	return FWindowsTimeInternal::CurrentThreadCPUStats.LastIntervalThreadTime;
}
