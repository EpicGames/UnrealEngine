// Copyright Epic Games, Inc. All Rights Reserved.

#include "WindowsRunnableThread.h"
#include "GenericPlatform/GenericPlatformCrashContext.h"
#include "HAL/ExceptionHandling.h"
#include "Misc/App.h"
#include "Misc/FeedbackContext.h"
#include "Misc/OutputDeviceError.h"
#include "Stats/Stats.h"

DEFINE_LOG_CATEGORY_STATIC(LogThreadingWindows, Log, All);

int FRunnableThreadWin::TranslateThreadPriority(EThreadPriority Priority)
{
	// If this triggers, 
	static_assert(TPri_Num == 7, "Need to add a case for new TPri_xxx enum value");

	switch (Priority)
	{
		case TPri_AboveNormal:			return THREAD_PRIORITY_ABOVE_NORMAL;
		case TPri_Normal:				return THREAD_PRIORITY_NORMAL;
		case TPri_BelowNormal:			return THREAD_PRIORITY_BELOW_NORMAL;
		case TPri_Highest:				return THREAD_PRIORITY_HIGHEST;
		case TPri_TimeCritical:			return THREAD_PRIORITY_HIGHEST;
		case TPri_Lowest:				return THREAD_PRIORITY_LOWEST;

		// There is no such things as slightly below normal on Windows.
		// This can't be below normal since we don't want latency sensitive task to go to efficient cores on Alder Lake.
		case TPri_SlightlyBelowNormal:	return THREAD_PRIORITY_NORMAL;

	// Note: previously, the behavior was:
	//
	//case TPri_AboveNormal:			return THREAD_PRIORITY_HIGHEST;
	//case TPri_Normal:					return THREAD_PRIORITY_HIGHEST - 1;
	//case TPri_BelowNormal:			return THREAD_PRIORITY_HIGHEST - 3;
	//case TPri_Highest:				return THREAD_PRIORITY_HIGHEST;
	//case TPri_TimeCritical:			return THREAD_PRIORITY_HIGHEST;
	//case TPri_Lowest:					return THREAD_PRIORITY_HIGHEST - 4;
	//case TPri_SlightlyBelowNormal:	return THREAD_PRIORITY_HIGHEST - 2;
	//
	// But the change (CL3747560) was not well documented (it didn't describe
	// the symptoms it was supposed to address) and introduces undesirable 
	// system behavior on Windows since it starves out other processes in
	// the system when UE compiles shaders or otherwise goes wide due to the
	// inflation in priority (Normal mapped to THREAD_PRIORITY_ABOVE_NORMAL)
	// I kept the TPri_TimeCritical mapping to THREAD_PRIORITY_HIGHEST however
	// to avoid introducing poor behavior since time critical priority is
	// similarly detrimental to overall system behavior.
	//
	// If we discover thread scheduling issues it would maybe be better to 
	// adjust actual thread priorities at the source instead of this mapping?

	default: UE_LOGF(LogHAL, Fatal, "Unknown Priority passed to TranslateThreadPriority()"); return THREAD_PRIORITY_NORMAL;
	}
}

uint32 FRunnableThreadWin::GuardedRun()
{
	uint32 ExitCode = 0;

	FPlatformProcess::SetThreadAffinityMask(ThreadAffinityMask);

	FPlatformProcess::SetThreadName(*ThreadName);

	// Initialize random number generator. srand() is thread-local in windows.
	if (!FApp::bUseFixedSeed)
	{
		FMath::RandInit(FPlatformTime::Cycles());
	}

	const TCHAR* CmdLine = ::GetCommandLineW();
	bool bNoExceptionHandler = FParse::Param(::GetCommandLineW(), TEXT("noexceptionhandler"));
#if UE_BUILD_DEBUG
	if (true && !GAlwaysReportCrash)
#else
	if (bNoExceptionHandler || (FPlatformMisc::IsDebuggerPresent() && !GAlwaysReportCrash))
#endif // UE_BUILD_DEBUG
	{
		ExitCode = Run();
	}
	else
	{
#if !PLATFORM_SEH_EXCEPTIONS_DISABLED
		__try
#endif // !PLATFORM_SEH_EXCEPTIONS_DISABLED
		{
			ExitCode = Run();
		}
#if !PLATFORM_SEH_EXCEPTIONS_DISABLED
		__except (FPlatformMisc::GetCrashHandlingType() == ECrashHandlingType::Default ? ReportCrash(GetExceptionInformation()) : EXCEPTION_CONTINUE_SEARCH)
		{
			__try
			{
				// Make sure the information which thread crashed makes it into the log.
				UE_LOGF( LogThreadingWindows, Error, "Runnable thread %ls crashed.", *ThreadName );
				GWarn->Flush();

				// Append the thread name at the end of the error report.
				FCString::StrncatTruncateDest( GErrorHist, UE_ARRAY_COUNT(GErrorHist), TEXT(LINE_TERMINATOR_ANSI "Crash in runnable thread " ) );
				FCString::StrncatTruncateDest( GErrorHist, UE_ARRAY_COUNT(GErrorHist), *ThreadName );

				// Crashed.
				ExitCode = 1;
				GError->HandleError();
				FPlatformMisc::RequestExit(true, TEXT("FRunnableThreadWin::GuardedRun.ExceptionHandler"));
			}
			__except(EXCEPTION_EXECUTE_HANDLER)
			{
				// The crash handler crashed itself, exit with a code which the 
				// out-of-process monitor will be able to pick up and report into 
				// analytics.
				
				TerminateProcess(GetCurrentProcess(), (UINT)ECrashExitCodes::CrashHandlerCrashed);
			}
		}
#endif // !PLATFORM_SEH_EXCEPTIONS_DISABLED
	}

	return ExitCode;
}

bool FRunnableThreadWin::SetThreadAffinity(const FThreadAffinity& Affinity)
{
	// While it's technically possible to use both process and thread affinities,
	// it requires restricting the set of cores eligible for affinity to respect 
	// the process affinity mask.
	// For simplicity, as long as the process-wide affinity is a debugging option,
	// disallow thread affinities when using process affinity.
	if (FWindowsPlatformProcess::IsProcessAffinitySet())
	{
		return false;
	}

	const FProcessorGroupDesc& ProcessorGroups = FPlatformMisc::GetProcessorGroupDesc();
	int32 CpuGroupCount = ProcessorGroups.NumProcessorGroups;
	check(Affinity.ProcessorGroup < CpuGroupCount);

	GROUP_AFFINITY GroupAffinity = {};
	GROUP_AFFINITY PreviousGroupAffinity = {};
	GroupAffinity.Mask = Affinity.ThreadAffinityMask & ProcessorGroups.ThreadAffinities[Affinity.ProcessorGroup];
	GroupAffinity.Group = Affinity.ProcessorGroup;
	if (SetThreadGroupAffinity(Thread, &GroupAffinity, &PreviousGroupAffinity) == 0)
	{
		DWORD LastError = GetLastError();
		UE_LOGF( LogThreadingWindows, Warning, "Runnable thread %ls call to SetThreadAffinity failed with: 0x%x", *ThreadName, LastError);
		return  false;
	}
	ThreadAffinityMask = Affinity.ThreadAffinityMask;
	return PreviousGroupAffinity.Mask != GroupAffinity.Mask || PreviousGroupAffinity.Group != GroupAffinity.Group;
};

uint32 FRunnableThreadWin::Run()
{
	uint32 ExitCode = 1;
	check(Runnable);

	if (Runnable->Init() == true)
	{
		ThreadInitSyncEvent->Trigger();

		// Setup TLS for this thread, used by FTlsAutoCleanup objects.
		SetTls();

		ExitCode = Runnable->Run();

		// Allow any allocated resources to be cleaned up
		Runnable->Exit();

#if STATS
		FThreadStats::Shutdown();
#endif
		FreeTls();
	}
	else
	{
		// Initialization has failed, release the sync event
		ThreadInitSyncEvent->Trigger();
	}

	return ExitCode;
}
