// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/Invoke.h"
#include "Templates/UnrealTemplate.h"

// TPlus<T> specifically takes const T& and returns T.
// TPlus<> (empty angle brackets) is late-binding, taking whatever is passed and returning the correct result type for (A+B)
template<typename T = void>
struct TPlus
{
	[[nodiscard]] UE_REWRITE T operator()(const T& A, const T& B) { return A + B; }
};
template<>
struct TPlus<void>
{
	template<typename U, typename V>
	[[nodiscard]] UE_REWRITE auto operator()(U&& A, V&& B) -> decltype(A + B) { return A + B; }
};


namespace Algo
{
	/**
	* Sums a range by successively applying Op.
	*
	* @param  Input  Any iterable type
	* @param  Init  Initial value for the summation
	* @param  Op  Summing Operation (the default is TPlus<>)
	*
	* @return the result of summing all the elements of Input
	*/
	template <typename T, typename A, typename OpT>
	[[nodiscard]] T Accumulate(const A& Input, T Init, OpT Op)
	{
		T Result = MoveTemp(Init);
		for (const auto& InputElem : Input)
		{
			Result = Invoke(Op, MoveTemp(Result), InputElem);
		}
		return Result;
	}

	/**
	 * Sums a range.
	 *
	 * @param  Input  Any iterable type
	 * @param  Init  Initial value for the summation
	 *
	 * @return the result of summing all the elements of Input
	 */
	template <typename T, typename A>
	[[nodiscard]] UE_REWRITE T Accumulate(const A& Input, T Init)
	{
		return Accumulate(Input, MoveTemp(Init), TPlus<>());
	}

	/**
	* Sums a range by applying MapOp to each element, and then summing the results.
	*
	* @param  Input  Any iterable type
	* @param  MapOp  Mapping Operation
	* @param  Init  Initial value for the summation
	* @param  Op  Summing Operation (the default is TPlus<>)
	*
	* @return the result of mapping and then summing all the elements of Input
	*/
	template <typename T, typename A, typename MapT, typename OpT>
	[[nodiscard]] T TransformAccumulate(const A& Input, MapT MapOp, T Init, OpT Op)
	{
		T Result = MoveTemp(Init);
		for (const auto& InputElem : Input)
		{
			Result = Invoke(Op, MoveTemp(Result), Invoke(MapOp, InputElem));
		}
		return Result;
	}

	/**
	* Sums a range by applying MapOp to each element, and then summing the results.
	*
	* @param  Input  Any iterable type
	* @param  MapOp  Mapping Operation
	* @param  Init  Initial value for the summation
	*
	* @return the result of mapping and then summing all the elements of Input
	*/
	template <typename T, typename A, typename MapT>
	[[nodiscard]] UE_REWRITE T TransformAccumulate(const A& Input, MapT MapOp, T Init)
	{
		return TransformAccumulate(Input, MapOp, MoveTemp(Init), TPlus<>());
	}
}
