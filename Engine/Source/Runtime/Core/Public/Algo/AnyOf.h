// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Algo/NoneOf.h"
#include "Templates/UnrealTemplate.h" // for MoveTemp


namespace Algo
{
	/**
	 * Checks if any element in the range is truthy.
	 *
	 * @param  Range  The range to check.
	 *
	 * @return  true if at least one element is truthy, false otherwise.
	 */
	template <typename RangeType>
	[[nodiscard]] UE_REWRITE bool AnyOf(const RangeType& Range)
	{
		return !Algo::NoneOf(Range);
	}

	/**
	 * Checks if any projection of the elements in the range is truthy.
	 *
	 * @param  Range       The range to check.
	 * @param  Projection  The projection to apply to each element.
	 *
	 * @return  true if at least one projection is truthy, false otherwise.
	 */
	template <typename RangeType, typename ProjectionType>
	[[nodiscard]] UE_REWRITE bool AnyOf(const RangeType& Range, ProjectionType Projection)
	{
		return !Algo::NoneOf(Range, MoveTemp(Projection));
	}

	/**
	 * Checks if any projection of the elements in the range is truthy.
	 * Should be used when the range iterator doesn't return a reference.
	 *
	 * @param  Range       The range to check.
	 * @param  Projection  The projection to apply to each element.
	 *
	 * @return  true if at least one projection is truthy, false otherwise.
	 */
	template <typename RangeType, typename ProjectionType>
	[[nodiscard]] UE_REWRITE bool AnyOf(const RangeType& Range, ProjectionType Projection, ENoRef NoRef)
	{
		return !Algo::NoneOf(Range, MoveTemp(Projection), NoRef);
	}
}
